// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	ParticleEmitterInstances.cpp: Particle emitter instance implementations.
=============================================================================*/

#include "ParticleEmitterInstances.h"
#include "ParticleEmitterInstanceOwner.h"
#include "Engine/Engine.h"
#include "Engine/World.h"
#include "Materials/Material.h"
#include "MaterialDomain.h"
#include "Materials/MaterialRelevance.h"
#include "Particles/ParticleSystem.h"
#include "Engine/StaticMesh.h"
#include "Particles/Orientation/ParticleModuleOrientationAxisLock.h"
#include "StaticMeshResources.h"
#include "FXSystem.h"


#include "Particles/ParticleEmitter.h"
#include "Particles/SubUV/ParticleModuleSubUV.h"
#include "Particles/Collision/ParticleModuleCollisionGPU.h"
#include "Particles/Event/ParticleModuleEventGenerator.h"
#include "Particles/Event/ParticleModuleEventReceiverBase.h"
#include "Particles/Material/ParticleModuleMeshMaterial.h"
#include "Particles/Modules/Location/ParticleModulePivotOffset.h"
#include "Particles/Orbit/ParticleModuleOrbit.h"
#include "Particles/ParticleModule.h"
#include "Particles/Spawn/ParticleModuleSpawn.h"
#include "Particles/ParticleSpriteEmitter.h"
#include "Particles/TypeData/ParticleModuleTypeDataBase.h"
#include "Particles/ParticleSystemComponent.h"
#include "Particles/TypeData/ParticleModuleTypeDataMesh.h"
#include "Particles/ParticleLODLevel.h"
#include "Particles/ParticleModuleRequired.h"

#include "Components/PointLightComponent.h"
#include "Particles/Spawn/ParticleModuleSpawnBase.h"
#include "Particles/SubUVAnimation.h"
#include "Stats/StatsTrace.h"

// Comment this in to debug empty emitter instance templates...
//#define _PSYSCOMP_DEBUG_INVALID_EMITTER_INSTANCE_TEMPLATES_

/*-----------------------------------------------------------------------------
FParticlesStatGroup
-----------------------------------------------------------------------------*/


DEFINE_STAT(STAT_ParticleDrawCalls);
DEFINE_STAT(STAT_SpriteParticles);
DEFINE_STAT(STAT_SpriteParticlesSpawned);
DEFINE_STAT(STAT_SpriteParticlesUpdated);
DEFINE_STAT(STAT_SpriteParticlesKilled);
DEFINE_STAT(STAT_SortingTime);
DEFINE_STAT(STAT_SpriteRenderingTime);
DEFINE_STAT(STAT_SpriteTickTime);
DEFINE_STAT(STAT_SpriteSpawnTime);
DEFINE_STAT(STAT_PSysCompTickTime);
DEFINE_STAT(STAT_ParticlePoolTime);
DEFINE_STAT(STAT_ParticleComputeTickTime);
DEFINE_STAT(STAT_ParticleFinalizeTickTime);
DEFINE_STAT(STAT_GTSTallTime);
DEFINE_STAT(STAT_ParticleRenderingTime);
DEFINE_STAT(STAT_ParticlePackingTime);
DEFINE_STAT(STAT_ParticleSetTemplateTime);
DEFINE_STAT(STAT_ParticleInitializeTime);
DEFINE_STAT(STAT_ParticleActivateTime);
DEFINE_STAT(STAT_ParticleUpdateBounds);
DEFINE_STAT(STAT_ParticleAsyncTime);
DEFINE_STAT(STAT_ParticleAsyncWaitTime);





DEFINE_STAT(STAT_MeshParticles);
DEFINE_STAT(STAT_MeshRenderingTime);
DEFINE_STAT(STAT_MeshTickTime);

DEFINE_STAT(STAT_MeshParticlePolys);

/** GPU Particle stats. */

DEFINE_STAT(STAT_GPUSpriteParticles);
DEFINE_STAT(STAT_GPUSpritesSpawned);
DEFINE_STAT(STAT_SortedGPUParticles);
DEFINE_STAT(STAT_SortedGPUEmitters);
DEFINE_STAT(STAT_FreeGPUTiles);
DEFINE_STAT(STAT_GPUParticleMisc3);
DEFINE_STAT(STAT_GPUParticleMisc2);
DEFINE_STAT(STAT_GPUParticleMisc1);
DEFINE_STAT(STAT_GPUParticleBuildSimCmdsTime);
DEFINE_STAT(STAT_GPUParticleTickTime);
DEFINE_STAT(STAT_GPUSpriteRenderingTime);
DEFINE_STAT(STAT_GPUSpritePreRenderTime);
DEFINE_STAT(STAT_GPUSpriteSpawnTime);
DEFINE_STAT(STAT_GPUSpriteTickTime);
DEFINE_STAT(STAT_GPUSingleIterationEmitters);
DEFINE_STAT(STAT_GPUMultiIterationsEmitters);
DEFINE_STAT(STAT_GPUParticlesInjectionTime);
DEFINE_STAT(STAT_GPUParticlesSimulationCommands);

/** Particle memory stats */

DEFINE_STAT(STAT_GTParticleData);
DEFINE_STAT(STAT_DynamicSpriteGTMem);
DEFINE_STAT(STAT_DynamicSubUVGTMem);
DEFINE_STAT(STAT_DynamicMeshGTMem);
DEFINE_STAT(STAT_DynamicBeamGTMem);
DEFINE_STAT(STAT_DynamicRibbonGTMem);
DEFINE_STAT(STAT_DynamicAnimTrailGTMem);
DEFINE_STAT(STAT_DynamicUntrackedGTMem);

DEFINE_STAT(STAT_RTParticleData);
DEFINE_STAT(STAT_GTParticleData_MAX);
DEFINE_STAT(STAT_RTParticleData_MAX);
DEFINE_STAT(STAT_RTParticleData_Largest);
DEFINE_STAT(STAT_RTParticleData_Largest_MAX);
DEFINE_STAT(STAT_DynamicPSysCompMem);
DEFINE_STAT(STAT_DynamicPSysCompMem_MAX);
DEFINE_STAT(STAT_DynamicPSysCompCount);
DEFINE_STAT(STAT_DynamicPSysCompCount_MAX);
DEFINE_STAT(STAT_DynamicEmitterMem);
DEFINE_STAT(STAT_DynamicEmitterMem_MAX);
DEFINE_STAT(STAT_DynamicEmitterCount);
DEFINE_STAT(STAT_DynamicEmitterCount_MAX);
DEFINE_STAT(STAT_DynamicEmitterGTMem_Waste);
DEFINE_STAT(STAT_DynamicEmitterGTMem_Waste_MAX);
DEFINE_STAT(STAT_DynamicEmitterGTMem_Largest);
DEFINE_STAT(STAT_DynamicEmitterGTMem_Largest_MAX);
DEFINE_STAT(STAT_DynamicSpriteCount);
DEFINE_STAT(STAT_DynamicSpriteCount_MAX);
DEFINE_STAT(STAT_DynamicSpriteGTMem_MAX);
DEFINE_STAT(STAT_DynamicSubUVCount);
DEFINE_STAT(STAT_DynamicSubUVCount_MAX);
DEFINE_STAT(STAT_DynamicSubUVGTMem_Max);
DEFINE_STAT(STAT_DynamicMeshCount);
DEFINE_STAT(STAT_DynamicMeshCount_MAX);
DEFINE_STAT(STAT_DynamicMeshGTMem_MAX);
DEFINE_STAT(STAT_DynamicBeamCount);
DEFINE_STAT(STAT_DynamicBeamCount_MAX);
DEFINE_STAT(STAT_DynamicBeamGTMem_MAX);
DEFINE_STAT(STAT_DynamicRibbonCount);
DEFINE_STAT(STAT_DynamicRibbonCount_MAX);
DEFINE_STAT(STAT_DynamicRibbonGTMem_MAX);
DEFINE_STAT(STAT_DynamicAnimTrailCount);
DEFINE_STAT(STAT_DynamicAnimTrailCount_MAX);
DEFINE_STAT(STAT_DynamicAnimTrailGTMem_MAX);
DEFINE_STAT(STAT_DynamicUntrackedGTMem_MAX);

DECLARE_CYCLE_STAT(TEXT("EmitterInstance Init GT"), STAT_ParticleEmitterInstance_Init, STATGROUP_Particles);
DECLARE_CYCLE_STAT(TEXT("MeshEmitterInstance Init GT"), STAT_MeshEmitterInstance_Init, STATGROUP_Particles);
DECLARE_CYCLE_STAT(TEXT("EmitterInstance InitParams GT"), STAT_ParticleEmitterInstance_InitParameters, STATGROUP_Particles);
DECLARE_CYCLE_STAT(TEXT("MeshEmitterInstance InitParams GT"), STAT_MeshEmitterInstance_InitParameters, STATGROUP_Particles);
DECLARE_CYCLE_STAT(TEXT("EmitterInstance Init Sizes GT"), STAT_ParticleEmitterInstance_InitSize, STATGROUP_Particles);
DECLARE_CYCLE_STAT(TEXT("EmitterInstance PrepPerInstanceBlock GT"), STAT_PrepPerInstanceBlock, STATGROUP_Particles);
DECLARE_CYCLE_STAT(TEXT("EmitterInstance Resize GT"), STAT_ParticleEmitterInstance_Resize, STATGROUP_Particles);


#define USE_FAST_PARTICLE_POOL 1
#if USE_FAST_PARTICLE_POOL

static int32 GEnableFastPools = 0;
static FAutoConsoleVariableRef CVarEnableFastPools(
	TEXT("r.Emitter.FastPoolEnable"),
	GEnableFastPools,
	TEXT("Should we use fast pools for emitters.\n")
	TEXT(" 0: Don't pool anything\n")
	TEXT(" 1: Pool the emitters bro (default)\n"),
	ECVF_Default
);

static int32 GMaxFreePoolSizeBytes = 2 * 1024 * 1024;
static FAutoConsoleVariableRef CVarFastPoolMaxFreeSize(
	TEXT("r.Emitter.FastPoolMaxFreeSize"),
	GMaxFreePoolSizeBytes,
	TEXT("Max free pool size to keep around without cleaning up."),
	ECVF_Default
);

FCriticalSection GFastPoolsCriticalSection;

struct FFastPoolFreePool
{
	FFastPoolFreePool() : LastUsedTime(0.0) { }

	TArray<void*> FreeAllocations;
	double LastUsedTime;
};

TMap<int32, FFastPoolFreePool> GFastPoolFreedAllocations;
int32 GFreePoolSizeBytes = 0;

#define FASTPARTICLEALLOC_CHECKSIZE 0
#if FASTPARTICLEALLOC_CHECKSIZE 
FCriticalSection GFastPoolSizeMapCriticalSection;
TMap<void*, int32> GFastPoolSizeMap;
#endif

FORCEINLINE static void* FastParticleSmallBlockAlloc(size_t AllocSize)
{
	if (GEnableFastPools)
	{
		FScopeLock S(&GFastPoolsCriticalSection);

		FFastPoolFreePool* Allocations = GFastPoolFreedAllocations.Find(AllocSize);
		if ( Allocations  && Allocations->FreeAllocations.Num() )
		{
			void* Result = Allocations->FreeAllocations[0];
			Allocations->FreeAllocations.RemoveAtSwap(0, EAllowShrinking::No);
			Allocations->LastUsedTime = FPlatformTime::Seconds();
			GFreePoolSizeBytes -= AllocSize;
#if FASTPARTICLEALLOC_CHECKSIZE
			FScopeLock Q(&GFastPoolSizeMapCriticalSection);
			GFastPoolSizeMap.Add(Result, AllocSize);
#endif
			return Result;
		}

	}

#if FASTPARTICLEALLOC_CHECKSIZE
	FScopeLock S(&GFastPoolSizeMapCriticalSection);
	void *Result = FMemory::Malloc(AllocSize);
	GFastPoolSizeMap.Add(Result, AllocSize);
	return Result;
#else
	return FMemory::Malloc(AllocSize);
#endif
}

FORCEINLINE static void FastParticleSmallBlockFree(void *RawMemory, size_t AllocSize)
{
#if FASTPARTICLEALLOC_CHECKSIZE
	{
		FScopeLock S(&GFastPoolSizeMapCriticalSection);
		int32* AllocatedSize = GFastPoolSizeMap.Find(RawMemory);
		GFastPoolSizeMap.Remove(RawMemory);
		check(AllocatedSize);
		if (AllocatedSize)
		{
			check(*AllocatedSize == AllocSize);
		}
	}
#endif
	if (GEnableFastPools)
	{
		FScopeLock S(&GFastPoolsCriticalSection);
		FFastPoolFreePool& Allocations = GFastPoolFreedAllocations.FindOrAdd(AllocSize);
		Allocations.FreeAllocations.Add(RawMemory);
		GFreePoolSizeBytes += AllocSize;

		if ( GFreePoolSizeBytes > GMaxFreePoolSizeBytes )
		{
			// free the oldest allocation
			FFastPoolFreePool* OldestPool = nullptr;
			int32 OldestPoolAllocSize  = 0;
			for ( auto& PoolItr : GFastPoolFreedAllocations )
			{
				FFastPoolFreePool& Pool = PoolItr.Value;
				if ( Pool.FreeAllocations.Num() )
				{
					if ( OldestPool == nullptr )
					{
						OldestPoolAllocSize = PoolItr.Key;
						OldestPool = &Pool;
					}
					else
					{
						if (OldestPool->LastUsedTime > Pool.LastUsedTime)
						{
							OldestPoolAllocSize = PoolItr.Key;
							OldestPool = &Pool;
						}
					}
				}
			}
			check( OldestPool );
			check( OldestPoolAllocSize  != 0 );
			void* OldAllocation = OldestPool->FreeAllocations[0];
			OldestPool->FreeAllocations.RemoveAtSwap(0, EAllowShrinking::No);
			GFreePoolSizeBytes -= OldestPoolAllocSize;
			FMemory::Free(OldAllocation);
		}
		return;
	}
	FMemory::Free(RawMemory);
}


#else
FORCEINLINE static void* FastParticleSmallBlockAlloc(size_t AllocSize)
{
	return FMemory::Malloc(AllocSize);
}

FORCEINLINE static void FastParticleSmallBlockFree(void *RawMemory, size_t AllocSize)
{
	FMemory::Free(RawMemory);
}

#endif



void FParticleDataContainer::Alloc(int32 InParticleDataNumBytes, int32 InParticleIndicesNumShorts)
{
	check(InParticleDataNumBytes > 0 && ParticleIndicesNumShorts >= 0
		&& InParticleDataNumBytes % sizeof(uint16) == 0); // we assume that the particle storage has reasonable alignment below
	ParticleDataNumBytes = InParticleDataNumBytes;
	ParticleIndicesNumShorts = InParticleIndicesNumShorts;

	MemBlockSize = ParticleDataNumBytes + ParticleIndicesNumShorts * sizeof(uint16);

	ParticleData = (uint8*)FastParticleSmallBlockAlloc(MemBlockSize);
	ParticleIndices = (uint16*)(ParticleData + ParticleDataNumBytes);
}

void FParticleDataContainer::Free()
{
	if (ParticleData)
	{
		check(MemBlockSize > 0);
		FastParticleSmallBlockFree(ParticleData, MemBlockSize);
	}
	MemBlockSize = 0;
	ParticleDataNumBytes = 0;
	ParticleIndicesNumShorts = 0;
	ParticleData = nullptr;
	ParticleIndices = nullptr;
}


/*-----------------------------------------------------------------------------
	Information compiled from modules to build runtime emitter data.
-----------------------------------------------------------------------------*/

FParticleEmitterBuildInfo::FParticleEmitterBuildInfo()
	: RequiredModule(NULL)
	, SpawnModule(NULL)
	, SpawnPerUnitModule(NULL)
	, MaxSize(1.0f, 1.0f)
	, SizeScaleBySpeed(FVector2D::ZeroVector)
	, MaxSizeScaleBySpeed(1.0f, 1.0f)
	, bEnableCollision(false)
	, CollisionResponse(EParticleCollisionResponse::Bounce)
	, CollisionMode(EParticleCollisionMode::SceneDepth)
	, CollisionRadiusScale(1.0f)
	, CollisionRadiusBias(0.0f)
	, CollisionRandomSpread(0.0f)
	, CollisionRandomDistribution(1.0f)
	, Friction(0.0f)
	, PointAttractorPosition(FVector::ZeroVector)
	, PointAttractorRadius(0.0f)
	, GlobalVectorFieldScale(0.0f)
	, GlobalVectorFieldTightness(-1)
	, LocalVectorField(NULL)
	, LocalVectorFieldTransform(FTransform::Identity)
	, LocalVectorFieldIntensity(0.0f)
	, LocalVectorFieldTightness(0.0f)
	, LocalVectorFieldMinInitialRotation(FVector::ZeroVector)
	, LocalVectorFieldMaxInitialRotation(FVector::ZeroVector)
	, LocalVectorFieldRotationRate(FVector::ZeroVector)
	, ConstantAcceleration(0.0f)
	, MaxLifetime(1.0f)
	, MaxRotationRate(1.0f)
	, EstimatedMaxActiveParticleCount(0)
	, ScreenAlignment(PSA_Square)
	, PivotOffset(-0.5,-0.5)
	, bLocalVectorFieldIgnoreComponentTransform(false)
	, bLocalVectorFieldTileX(false)
	, bLocalVectorFieldTileY(false)
	, bLocalVectorFieldTileZ(false)
	, bLocalVectorFieldUseFixDT(false)
	, bUseVelocityForMotionBlur(0)
	, bRemoveHMDRoll(0)
	, MinFacingCameraBlendDistance(0.0f)
	, MaxFacingCameraBlendDistance(0.0f)
{
	DragScale.InitializeWithConstant(1.0f);
	VectorFieldScale.InitializeWithConstant(1.0f);
	VectorFieldScaleOverLife.InitializeWithConstant(1.0f);
#if WITH_EDITOR
	DynamicColorScale.Initialize();
	DynamicAlphaScale.Initialize();
#endif
}

/*-----------------------------------------------------------------------------
	FParticleEmitterInstance
-----------------------------------------------------------------------------*/
// Only update the PeakActiveParticles if the frame rate is 20 or better
const float FParticleEmitterInstance::PeakActiveParticleUpdateDelta = 0.05f;

/** Constructor	*/
FParticleEmitterInstance::FParticleEmitterInstance(IParticleEmitterInstanceOwner& InComponent) :
	  SpriteTemplate(NULL)
    , Component(InComponent)
    , CurrentLODLevel(NULL)
    , CurrentLODLevelIndex(0)
    , TypeDataOffset(0)
	, TypeDataInstanceOffset(-1)
    , SubUVDataOffset(0)
	, DynamicParameterDataOffset(0)
	, LightDataOffset(0)
	, LightVolumetricScatteringIntensity(0)
	, OrbitModuleOffset(0)
	, CameraPayloadOffset(0)
    , PayloadOffset(0)
	, bEnabled(1)
    , bKillOnDeactivate(0)
    , bKillOnCompleted(0)
	, bHaltSpawning(0)
	, bHaltSpawningExternal(0)
	, bRequiresLoopNotification(0)
	, bIgnoreComponentScale(0)
	, bIsBeam(0)
	, bAxisLockEnabled(0)
	, bFakeBurstsWhenSpawningSupressed(0)
	, LockAxisFlags(EPAL_NONE)
	, SortMode(PSORTMODE_None)
    , ParticleData(NULL)
    , ParticleIndices(NULL)
    , InstanceData(NULL)
    , InstancePayloadSize(0)
    , ParticleSize(0)
    , ParticleStride(0)
    , ActiveParticles(0)
	, ParticleCounter(0)
    , MaxActiveParticles(0)
    , SpawnFraction(0.0f)
    , SecondsSinceCreation(0.0f)
    , EmitterTime(0.0f)
    , LoopCount(0)
	, IsRenderDataDirty(0)
    , EmitterDuration(0.0f)
	, TrianglesToRender(0)
	, MaxVertexIndex(0)
	, CurrentMaterial(NULL)
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	, EventCount(0)
	, MaxEventCount(0)
#endif	//#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	, PositionOffsetThisTick(0)
	, PivotOffset(-0.5f,-0.5f)
{
}

/** Destructor	*/
FParticleEmitterInstance::~FParticleEmitterInstance()
{
	for (int32 i = 0; i < HighQualityLights.Num(); ++i)
	{
		UPointLightComponent* PointLightComponent = HighQualityLights[i];
		{
			PointLightComponent->Modify();
			PointLightComponent->DestroyComponent(false);
		}
	}
	HighQualityLights.Reset();

	FMemory::Free(ParticleData);
	FMemory::Free(ParticleIndices);
	FMemory::Free(InstanceData);
	BurstFired.Empty();
}

#if STATS
void FParticleEmitterInstance::PreDestructorCall()
{
	// Update the memory stat
	int32 TotalMem = (MaxActiveParticles * ParticleStride) + (MaxActiveParticles * sizeof(uint16));
	DEC_DWORD_STAT_BY(STAT_GTParticleData, TotalMem);
}
#endif

void FParticleEmitterInstance::InitParameters(UParticleEmitter* InTemplate)
{
	SCOPE_CYCLE_COUNTER(STAT_ParticleEmitterInstance_InitParameters);

	SpriteTemplate = InTemplate;
	SetupEmitterDuration();
}

/**
 *	Initialize the instance
 */
void FParticleEmitterInstance::Init()
{
	SCOPE_CYCLE_COUNTER(STAT_ParticleEmitterInstance_Init);

	check(SpriteTemplate != nullptr);

	// Use highest LOD level for init'ing data, will contain all module types.
	UParticleLODLevel* HighLODLevel = SpriteTemplate->LODLevels[0];

	// Set the current material
	check(HighLODLevel->RequiredModule);
	CurrentMaterial = HighLODLevel->RequiredModule->Material;

	// If we already have a non-zero ParticleSize, don't need to do most allocation work again
	bool bNeedsInit = (ParticleSize == 0);

	if(bNeedsInit)
	{
		SCOPE_CYCLE_COUNTER(STAT_ParticleEmitterInstance_InitSize);

		// Copy pre-calculated info
		bRequiresLoopNotification = SpriteTemplate->bRequiresLoopNotification;
		bAxisLockEnabled = SpriteTemplate->bAxisLockEnabled;
		LockAxisFlags = SpriteTemplate->LockAxisFlags;
		DynamicParameterDataOffset = SpriteTemplate->DynamicParameterDataOffset;
		LightDataOffset = SpriteTemplate->LightDataOffset;
		LightVolumetricScatteringIntensity = SpriteTemplate->LightVolumetricScatteringIntensity;
		CameraPayloadOffset = SpriteTemplate->CameraPayloadOffset;
		ParticleSize = SpriteTemplate->ParticleSize;
		PivotOffset = SpriteTemplate->PivotOffset;
		TypeDataOffset = SpriteTemplate->TypeDataOffset;
		TypeDataInstanceOffset = SpriteTemplate->TypeDataInstanceOffset;

	    if ((InstanceData == NULL) || (SpriteTemplate->ReqInstanceBytes > InstancePayloadSize))
	    {
			    InstanceData = (uint8*)(FMemory::Realloc(InstanceData, SpriteTemplate->ReqInstanceBytes));
			    InstancePayloadSize = SpriteTemplate->ReqInstanceBytes;
	    }
    
	    FMemory::Memzero(InstanceData, InstancePayloadSize);
    
	    for (UParticleModule* ParticleModule : SpriteTemplate->ModulesNeedingInstanceData)
	    {
			check(ParticleModule);
			uint8* PrepInstData = GetModuleInstanceData(ParticleModule);
			check(PrepInstData != nullptr); // Shouldn't be in the list if it doesn't have data
			ParticleModule->PrepPerInstanceBlock(this, (void*)PrepInstData);
	    }

		for (UParticleModule* ParticleModule : SpriteTemplate->ModulesNeedingRandomSeedInstanceData)
		{
			check(ParticleModule);
			FParticleRandomSeedInstancePayload* SeedInstancePayload = GetModuleRandomSeedInstanceData(ParticleModule);
			check(SeedInstancePayload != nullptr); // Shouldn't be in the list if it doesn't have data
			FParticleRandomSeedInfo* RandomSeedInfo = ParticleModule->GetRandomSeedInfo();
			ParticleModule->PrepRandomSeedInstancePayload(this, SeedInstancePayload, RandomSeedInfo ? *RandomSeedInfo : FParticleRandomSeedInfo());
		}

	    // Offset into emitter specific payload (e.g. TrailComponent requires extra bytes).
	    PayloadOffset = ParticleSize;
	    
	    // Update size with emitter specific size requirements.
	    ParticleSize += RequiredBytes();
    
	    // Make sure everything is at least 16 byte aligned so we can use SSE for FVector.
	    ParticleSize = Align(ParticleSize, 16);
    
	    // E.g. trail emitters store trailing particles directly after leading one.
	    ParticleStride = CalculateParticleStride(ParticleSize);
	}

	// Setup the emitter instance material array...
	SetMeshMaterials(SpriteTemplate->MeshMaterials);

	// Set initial values.
	SpawnFraction			= 0;
	SecondsSinceCreation	= 0;
	EmitterTime				= 0;
	ParticleCounter			= 0;

	UpdateTransforms();	
	Location				= Component.GetComponentTransform().GetLocation();
	OldLocation				= Location;
	
	TrianglesToRender		= 0;
	MaxVertexIndex			= 0;

	if (ParticleData == NULL)
	{
		MaxActiveParticles	= 0;
		ActiveParticles		= 0;
	}

	ParticleBoundingBox.Init();
	if (HighLODLevel->RequiredModule->RandomImageChanges == 0)
	{
		HighLODLevel->RequiredModule->RandomImageTime	= 1.0f;
	}
	else
	{
		HighLODLevel->RequiredModule->RandomImageTime	= 0.99f / (HighLODLevel->RequiredModule->RandomImageChanges + 1);
	}

	// Resize to sensible default.
	if (bNeedsInit && 
		Component.IsGameWorld() == true &&
		// Only presize if any particles will be spawned 
		SpriteTemplate->QualityLevelSpawnRateScale > 0)
	{
		if ((HighLODLevel->PeakActiveParticles > 0) || (SpriteTemplate->InitialAllocationCount > 0))
		{
			// In-game... we assume the editor has set this properly, but still clamp at 100 to avoid wasting
			// memory.
			if (SpriteTemplate->InitialAllocationCount > 0)
			{
				Resize(FMath::Min( SpriteTemplate->InitialAllocationCount, 100 ));
			}
			else
			{
				Resize(FMath::Min( HighLODLevel->PeakActiveParticles, 100 ));
			}
		}
		else
		{
			// This is to force the editor to 'select' a value
			Resize(10);
		}
	}

	LoopCount = 0;

	if(bNeedsInit)
	{
		//QUICK_SCOPE_CYCLE_COUNTER(STAT_AllocateBurstLists);
	// Propagate killon flags
		bKillOnDeactivate = HighLODLevel->RequiredModule->bKillOnDeactivate;
		bKillOnCompleted = HighLODLevel->RequiredModule->bKillOnCompleted;

	// Propagate sorting flag.
		SortMode = HighLODLevel->RequiredModule->SortMode;

	// Reset the burst lists
	if (BurstFired.Num() < SpriteTemplate->LODLevels.Num())
	{
		BurstFired.AddZeroed(SpriteTemplate->LODLevels.Num() - BurstFired.Num());
	}

	for (int32 LODIndex = 0; LODIndex < SpriteTemplate->LODLevels.Num(); LODIndex++)
	{
			UParticleLODLevel* LODLevel = SpriteTemplate->LODLevels[LODIndex];
		check(LODLevel);
		FLODBurstFired& LocalBurstFired = BurstFired[LODIndex];
		if (LocalBurstFired.Fired.Num() < LODLevel->SpawnModule->BurstList.Num())
		{
			LocalBurstFired.Fired.AddZeroed(LODLevel->SpawnModule->BurstList.Num() - LocalBurstFired.Fired.Num());
		}
	}
	}

	ResetBurstList();

#if WITH_EDITORONLY_DATA
	//Check for SubUV module to see if it has SubUVAnimation to move data to required module
	for (auto CurrModule : HighLODLevel->Modules)
	{
		if (CurrModule->IsA(UParticleModuleSubUV::StaticClass()))
		{
			UParticleModuleSubUV* SubUVModule = (UParticleModuleSubUV*)CurrModule;

			if (SubUVModule->Animation)
			{
				HighLODLevel->RequiredModule->AlphaThreshold = SubUVModule->Animation->AlphaThreshold;
				HighLODLevel->RequiredModule->BoundingMode = SubUVModule->Animation->BoundingMode;
				HighLODLevel->RequiredModule->OpacitySourceMode = SubUVModule->Animation->OpacitySourceMode;
				HighLODLevel->RequiredModule->CutoutTexture = SubUVModule->Animation->SubUVTexture;

				SubUVModule->Animation = nullptr;

				HighLODLevel->RequiredModule->CacheDerivedData();
				HighLODLevel->RequiredModule->InitBoundingGeometryBuffer();
			}
		}
	}
#endif //WITH_EDITORONLY_DATA

	// Tag it as dirty w.r.t. the renderer
	IsRenderDataDirty	= 1;

	bEmitterIsDone = false;
}

void FParticleEmitterInstance::UpdateTransforms()
{
	//QUICK_SCOPE_CYCLE_COUNTER(STAT_EmitterInstance_UpdateTransforms);

	check(SpriteTemplate != NULL);

	UParticleLODLevel* LODLevel = GetCurrentLODLevelChecked();
	FMatrix ComponentToWorld = Component.GetComponentToWorld().ToMatrixNoScale();
	FMatrix EmitterToComponent = FRotationTranslationMatrix(
		LODLevel->RequiredModule->EmitterRotation,
		LODLevel->RequiredModule->EmitterOrigin
		);

	if (LODLevel->RequiredModule->bUseLocalSpace)
	{
		EmitterToSimulation = EmitterToComponent;
		SimulationToWorld = ComponentToWorld;
#if ENABLE_NAN_DIAGNOSTIC
		if (SimulationToWorld.ContainsNaN())
		{
			logOrEnsureNanError(TEXT("FParticleEmitterInstance::UpdateTransforms() - SimulationToWorld contains NaN!"));
			SimulationToWorld = FMatrix::Identity;
		}
#endif
	}
	else
	{
		EmitterToSimulation = EmitterToComponent * ComponentToWorld;
		SimulationToWorld = FMatrix::Identity;
	}
}

/**
 * Ensures enough memory is allocated for the requested number of particles.
 *
 * @param NewMaxActiveParticles		The number of particles for which memory must be allocated.
 * @param bSetMaxActiveCount		If true, update the peak active particles for this LOD.
 * @returns bool					true if memory is allocated for at least NewMaxActiveParticles.
 */
bool FParticleEmitterInstance::Resize(int32 NewMaxActiveParticles, bool bSetMaxActiveCount)
{
	SCOPE_CYCLE_COUNTER(STAT_ParticleEmitterInstance_Resize);

	if (GEngine->MaxParticleResize > 0)
	{
		if ((NewMaxActiveParticles < 0) || (NewMaxActiveParticles > GEngine->MaxParticleResize))
		{
			if ((NewMaxActiveParticles < 0) || (NewMaxActiveParticles > GEngine->MaxParticleResizeWarn))
			{
				UE_LOGF(LogParticles, Warning, "Emitter::Resize> Invalid NewMaxActive (%d) for Emitter in PSys %ls",
					NewMaxActiveParticles, 
								Component.GetTemplate() ? *(Component.GetTemplate()->GetPathName()) 
													: *(Component.GetName()));
			}

			return false;
		}
	}

	if (NewMaxActiveParticles > MaxActiveParticles)
	{
		// Alloc (or realloc) the data array
		// Allocations > 16 byte are always 16 byte aligned so ParticleData can be used with SSE.
		// NOTE: We don't have to zero the memory here... It gets zeroed when grabbed later.
#if STATS
		{
			// Update the memory stat
			int32 OldMem = (MaxActiveParticles * ParticleStride) + (MaxActiveParticles * sizeof(uint16));
			int32 NewMem = (NewMaxActiveParticles * ParticleStride) + (NewMaxActiveParticles * sizeof(uint16));
			DEC_DWORD_STAT_BY(STAT_GTParticleData, OldMem);
			INC_DWORD_STAT_BY(STAT_GTParticleData, NewMem);
		}
#endif

		{
			ParticleData = (uint8*) FMemory::Realloc(ParticleData, ParticleStride * NewMaxActiveParticles);
			check(ParticleData);

			// Allocate memory for indices.
			if (ParticleIndices == NULL)
			{
				// Make sure that we clear all when it is the first alloc
				MaxActiveParticles = 0;
			}
			ParticleIndices	= (uint16*) FMemory::Realloc(ParticleIndices, sizeof(uint16) * (NewMaxActiveParticles + 1));
		}

		// Fill in default 1:1 mapping.
		for (int32 i=MaxActiveParticles; i<NewMaxActiveParticles; i++)
		{
			ParticleIndices[i] = i;
		}

		// Set the max count
		MaxActiveParticles = NewMaxActiveParticles;
	}

#if STATS
	{
		int32 WastedMem = 
			((MaxActiveParticles * ParticleStride) + (MaxActiveParticles * sizeof(uint16))) - 
			((ActiveParticles * ParticleStride) + (ActiveParticles * sizeof(uint16)));
		INC_DWORD_STAT_BY(STAT_DynamicEmitterGTMem_Waste,WastedMem);
	}
#endif

	// Set the PeakActiveParticles
	if (bSetMaxActiveCount)
	{
		UParticleLODLevel* LODLevel	= SpriteTemplate->GetLODLevel(0);
		check(LODLevel);
		if (MaxActiveParticles > LODLevel->PeakActiveParticles)
		{
			LODLevel->PeakActiveParticles = MaxActiveParticles;
		}
	}

	return true;
}

/**
 *	Tick the instance.
 *
 *	@param	DeltaTime			The time slice to use
 *	@param	bSuppressSpawning	If true, do not spawn during Tick
 */
void FParticleEmitterInstance::Tick(float DeltaTime, bool bSuppressSpawning)
{
	SCOPE_CYCLE_COUNTER(STAT_SpriteTickTime);

	check(SpriteTemplate);
	check(SpriteTemplate->LODLevels.Num() > 0);

	// If this the FirstTime we are being ticked?
	bool bFirstTime = (SecondsSinceCreation > 0.0f) ? false : true;

	// Grab the current LOD level
	UParticleLODLevel* LODLevel = GetCurrentLODLevelChecked();

	// Handle EmitterTime setup, looping, etc.
	float EmitterDelay = Tick_EmitterTimeSetup(DeltaTime, LODLevel);

	if (bEnabled)
	{
		// Kill off any dead particles
		KillParticles();

		// Reset particle parameters.
		ResetParticleParameters(DeltaTime);

		// Update the particles
		CurrentMaterial = LODLevel->RequiredModule->Material;
		Tick_ModuleUpdate(DeltaTime, LODLevel);

		// Spawn new particles.
		SpawnFraction = Tick_SpawnParticles(DeltaTime, LODLevel, bSuppressSpawning, bFirstTime);

		// PostUpdate (beams only)
		Tick_ModulePostUpdate(DeltaTime, LODLevel);

		if (ActiveParticles > 0)
		{
			// Update the orbit data...
			UpdateOrbitData(DeltaTime);
			// Calculate bounding box and simulate velocity.
			UpdateBoundingBox(DeltaTime);
		}

		Tick_ModuleFinalUpdate(DeltaTime, LODLevel);

		CheckEmitterFinished();

		// Invalidate the contents of the vertex/index buffer.
		IsRenderDataDirty = 1;
	}
	else
	{
		FakeBursts();
	}

	// 'Reset' the emitter time so that the delay functions correctly
	EmitterTime += EmitterDelay;

	// Store the last delta time.
	LastDeltaTime = DeltaTime;

	// Reset particles position offset
	PositionOffsetThisTick = FVector::ZeroVector;

	INC_DWORD_STAT_BY(STAT_SpriteParticles, ActiveParticles);
}


/**
*	Called from Tick to determine whether the emitter will no longer spawn particles
*   checks for emitters with 0 loops, infinite lifetime, and no continuous spawning (only bursts)
*	and sets bEmitterIsDone if the last burst lies in the past and there are no active particles
*	bEmitterIsDone is checked for all emitters by ParticleSystemComponent tick, and the particle
*	system is deactivated if it's true for all emitters, and if bAutoDeactivate is set on the ParticleSystem
*/
void FParticleEmitterInstance::CheckEmitterFinished()
{
	// Grab the current LOD level
	UParticleLODLevel* LODLevel = GetCurrentLODLevelChecked();

	// figure out if this emitter will no longer spawn particles
	//
	if (this->ActiveParticles == 0)
	{
		UParticleModuleSpawn *SpawnModule = LODLevel->SpawnModule;
		check(SpawnModule);

		FParticleBurst *LastBurst = nullptr;
		if (SpawnModule->BurstList.Num())
		{
			LastBurst = &SpawnModule->BurstList.Last();
		}

		if (!LastBurst || LastBurst->Time < this->EmitterTime)
		{
			const UParticleModuleRequired* RequiredModule = LODLevel->RequiredModule;
			check(RequiredModule);

			if (HasCompleted() || 
				(SpawnModule->GetMaximumSpawnRate() == 0
				&& RequiredModule->EmitterDuration == 0
				&& RequiredModule->EmitterLoops == 0)
				)
			{
				bEmitterIsDone = true;
			}
		}
	}
}


/**
 *	Tick sub-function that handle EmitterTime setup, looping, etc.
 *
 *	@param	DeltaTime			The current time slice
 *	@param	CurrentLODLevel		The current LOD level for the instance
 *
 *	@return	float				The EmitterDelay
 */
float FParticleEmitterInstance::Tick_EmitterTimeSetup(float DeltaTime, UParticleLODLevel* InCurrentLODLevel)
{
	// Make sure we don't try and do any interpolation on the first frame we are attached (OldLocation is not valid in this circumstance)
	if (Component.GetJustRegistered())
	{
		Location	= Component.GetComponentTransform().GetLocation();
		OldLocation	= Location;
	}
	else
	{
		// Keep track of location for world- space interpolation and other effects.
		OldLocation	= Location;
		Location	= Component.GetComponentTransform().GetLocation();
	}

	UpdateTransforms();
	SecondsSinceCreation += DeltaTime;

	// Update time within emitter loop.
	bool bLooped = false;
	if (InCurrentLODLevel->RequiredModule->bUseLegacyEmitterTime == false)
	{
		EmitterTime += DeltaTime;
		bLooped = (EmitterDuration > 0.0f) && (EmitterTime >= EmitterDuration);
	}
	else
	{
		EmitterTime = SecondsSinceCreation;
		if (EmitterDuration > UE_KINDA_SMALL_NUMBER)
		{
			EmitterTime = FMath::Fmod(SecondsSinceCreation, EmitterDuration);
			bLooped = ((SecondsSinceCreation - (EmitterDuration * LoopCount)) >= EmitterDuration);
		}
	}

	// Get the emitter delay time
	float EmitterDelay = CurrentDelay;

	// Determine if the emitter has looped
	if (bLooped)
	{
		LoopCount++;
		ResetBurstList();
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
		// Reset the event count each loop...
		if (EventCount > MaxEventCount)
		{
			MaxEventCount = EventCount;
		}
		EventCount = 0;
#endif	//#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)

		if (InCurrentLODLevel->RequiredModule->bUseLegacyEmitterTime == false)
		{
			EmitterTime -= EmitterDuration;
		}

		if ((InCurrentLODLevel->RequiredModule->bDurationRecalcEachLoop == true)
			|| ((InCurrentLODLevel->RequiredModule->bDelayFirstLoopOnly == true) && (LoopCount == 1))
			)
		{
			SetupEmitterDuration();
		}

		if (bRequiresLoopNotification == true)
		{
			for (int32 ModuleIdx = -3; ModuleIdx < InCurrentLODLevel->Modules.Num(); ModuleIdx++)
			{
				int32 ModuleFetchIdx;
				switch (ModuleIdx)
				{
				case -3:	ModuleFetchIdx = INDEX_REQUIREDMODULE;	break;
				case -2:	ModuleFetchIdx = INDEX_SPAWNMODULE;		break;
				case -1:	ModuleFetchIdx = INDEX_TYPEDATAMODULE;	break;
				default:	ModuleFetchIdx = ModuleIdx;				break;
				}

				UParticleModule* Module = InCurrentLODLevel->GetModuleAtIndex(ModuleFetchIdx);
				if (Module != NULL)
				{
					if (Module->RequiresLoopingNotification() == true)
					{
						Module->EmitterLoopingNotify(this);
					}
				}
			}
		}
	}

	// Don't delay unless required
	if ((InCurrentLODLevel->RequiredModule->bDelayFirstLoopOnly == true) && (LoopCount > 0))
	{
		EmitterDelay = 0;
	}

	// 'Reset' the emitter time so that the modules function correctly
	EmitterTime -= EmitterDelay;

	return EmitterDelay;
}

/**
 *	Tick sub-function that handles spawning of particles
 *
 *	@param	DeltaTime			The current time slice
 *	@param	CurrentLODLevel		The current LOD level for the instance
 *	@param	bSuppressSpawning	true if spawning has been suppressed on the owning particle system component
 *	@param	bFirstTime			true if this is the first time the instance has been ticked
 *
 *	@return	float				The SpawnFraction remaining
 */
float FParticleEmitterInstance::Tick_SpawnParticles(float DeltaTime, UParticleLODLevel* InCurrentLODLevel, bool bSuppressSpawning, bool bFirstTime)
{
	if (!bHaltSpawning && !bHaltSpawningExternal && !bSuppressSpawning && (EmitterTime >= 0.0f))
	{
		// If emitter is not done - spawn at current rate.
		// If EmitterLoops is 0, then we loop forever, so always spawn.
		if ((InCurrentLODLevel->RequiredModule->EmitterLoops == 0) ||
			(LoopCount < InCurrentLODLevel->RequiredModule->EmitterLoops) ||
			(SecondsSinceCreation < (EmitterDuration * InCurrentLODLevel->RequiredModule->EmitterLoops)) ||
			bFirstTime)
		{
			bFirstTime = false;
			SpawnFraction = Spawn(DeltaTime);
		}
	}
	else if (bFakeBurstsWhenSpawningSupressed)
	{
		FakeBursts();
	}
	
	return SpawnFraction;
}

/**
 *	Tick sub-function that handles module updates
 *
 *	@param	DeltaTime			The current time slice
 *	@param	CurrentLODLevel		The current LOD level for the instance
 */
void FParticleEmitterInstance::Tick_ModuleUpdate(float DeltaTime, UParticleLODLevel* InCurrentLODLevel)
{
	UParticleLODLevel* HighestLODLevel = SpriteTemplate->LODLevels[0];
	check(HighestLODLevel);
	for (int32 ModuleIndex = 0; ModuleIndex < InCurrentLODLevel->UpdateModules.Num(); ModuleIndex++)
	{
		UParticleModule* CurrentModule = InCurrentLODLevel->UpdateModules[ModuleIndex];
		if (CurrentModule && CurrentModule->bEnabled && CurrentModule->bUpdateModule)
		{
			CurrentModule->Update({ *this, (int32)GetModuleDataOffset(HighestLODLevel->UpdateModules[ModuleIndex]), DeltaTime });
		}
	}
}

/**
 *	Tick sub-function that handles module post updates
 *
 *	@param	DeltaTime			The current time slice
 *	@param	CurrentLODLevel		The current LOD level for the instance
 */
void FParticleEmitterInstance::Tick_ModulePostUpdate(float DeltaTime, UParticleLODLevel* InCurrentLODLevel)
{
	// Handle the TypeData module
	if (InCurrentLODLevel->TypeDataModule)
	{
		InCurrentLODLevel->TypeDataModule->Update({ *this, TypeDataOffset, DeltaTime });
	}
}

/**
 *	Tick sub-function that handles module FINAL updates
 *
 *	@param	DeltaTime			The current time slice
 *	@param	CurrentLODLevel		The current LOD level for the instance
 */
void FParticleEmitterInstance::Tick_ModuleFinalUpdate(float DeltaTime, UParticleLODLevel* InCurrentLODLevel)
{
	UParticleLODLevel* HighestLODLevel = SpriteTemplate->LODLevels[0];
	check(HighestLODLevel);
	for (int32 ModuleIndex = 0; ModuleIndex < InCurrentLODLevel->UpdateModules.Num(); ModuleIndex++)
	{
		UParticleModule* CurrentModule = InCurrentLODLevel->UpdateModules[ModuleIndex];
		if (CurrentModule && CurrentModule->bEnabled && CurrentModule->bFinalUpdateModule)
		{
			CurrentModule->FinalUpdate({ *this, (int32)GetModuleDataOffset(HighestLODLevel->UpdateModules[ModuleIndex]), DeltaTime });
		}
	}


	if (InCurrentLODLevel->TypeDataModule && InCurrentLODLevel->TypeDataModule->bEnabled && InCurrentLODLevel->TypeDataModule->bFinalUpdateModule)
	{
		InCurrentLODLevel->TypeDataModule->FinalUpdate({ *this, (int32)GetModuleDataOffset(HighestLODLevel->TypeDataModule), DeltaTime });
	}
}

/**
 *	Set the LOD to the given index
 *
 *	@param	InLODIndex			The index of the LOD to set as current
 *	@param	bInFullyProcess		If true, process burst lists, etc.
 */
void FParticleEmitterInstance::SetCurrentLODIndex(int32 InLODIndex, bool bInFullyProcess)
{
	if (SpriteTemplate != NULL)
	{
		CurrentLODLevelIndex = InLODIndex;
		// check to make certain the data in the content actually represents what we are being asked to render
		if (SpriteTemplate->LODLevels.Num() > CurrentLODLevelIndex)
		{
			CurrentLODLevel	= SpriteTemplate->LODLevels[CurrentLODLevelIndex];
		}
		// set to the LOD which is guaranteed to exist
		else
		{
			CurrentLODLevelIndex = 0;
			CurrentLODLevel = SpriteTemplate->LODLevels[CurrentLODLevelIndex];
		}
		EmitterDuration = EmitterDurations[CurrentLODLevelIndex];

		check(CurrentLODLevel);
		check(CurrentLODLevel->RequiredModule);

		if (bInFullyProcess == true)
		{
			bKillOnCompleted = CurrentLODLevel->RequiredModule->bKillOnCompleted;
			bKillOnDeactivate = CurrentLODLevel->RequiredModule->bKillOnDeactivate;

			// Check for bursts that should have been fired already...
			UParticleModuleSpawn* SpawnModule = CurrentLODLevel->SpawnModule;
			FLODBurstFired* LocalBurstFired = NULL;

			if (CurrentLODLevelIndex + 1 > BurstFired.Num())
			{
				// This should not happen, but catch it just in case...
				BurstFired.AddZeroed(CurrentLODLevelIndex - BurstFired.Num() + 1);
			}
			LocalBurstFired = &(BurstFired[CurrentLODLevelIndex]);

			if (LocalBurstFired->Fired.Num() < SpawnModule->BurstList.Num())
			{
				LocalBurstFired->Fired.AddZeroed(SpawnModule->BurstList.Num() - LocalBurstFired->Fired.Num());
			}

			for (int32 BurstIndex = 0; BurstIndex < SpawnModule->BurstList.Num(); BurstIndex++)
			{
				if (CurrentLODLevel->RequiredModule->EmitterDelay + SpawnModule->BurstList[BurstIndex].Time < EmitterTime)
				{
					LocalBurstFired->Fired[BurstIndex] = true;
				}
			}
		}

		if ((Component.IsGameWorld() == true) && (CurrentLODLevel->bEnabled == false))
		{
			// Kill active particles...
			KillParticlesForced();
		}
	}
	else
	{
		// This is a legitimate case when PSysComponents are cached...
		// However, with the addition of the bIsActive flag to that class, this should 
		// never be called when the component has not had it's instances initialized/activated.
#if defined(_PSYSCOMP_DEBUG_INVALID_EMITTER_INSTANCE_TEMPLATES_)
		// need better debugging here
		UE_LOGF(LogParticles, Warning, "Template of emitter instance %d (%ls) a ParticleSystemComponent (%ls) was NULL: %ls", 
			i, *GetName(), *Template->GetName(), *this->GetFullName());
#endif	//#if defined(_PSYSCOMP_DEBUG_INVALID_EMITTER_INSTANCE_TEMPLATES_)
	}
}

/**
 *	Rewind the instance.
 */
void FParticleEmitterInstance::Rewind()
{
	if (Component.HasWorld())
	{
		UE_LOGF(LogParticles,Verbose,
			"FParticleEmitterInstance::Rewind @ %fs %ls", Component.GetWorldTimeSeconds(),
			(SpriteTemplate != NULL && SpriteTemplate->GetOuter() != NULL) ? *SpriteTemplate->GetOuter()->GetName() : TEXT("NULL"));
	}

	SecondsSinceCreation = 0;
	EmitterTime = 0;
	LoopCount = 0;
	ParticleCounter = 0;
	bEnabled = 1;
	ResetBurstList();
}

/**
 *	Retrieve the bounding box for the instance
 *
 *	@return	FBox	The bounding box
 */
FBox FParticleEmitterInstance::GetBoundingBox()
{ 
	return ParticleBoundingBox;
}

int32 FParticleEmitterInstance::GetOrbitPayloadOffset()
{
	UParticleLODLevel* LODLevel = GetCurrentLODLevelChecked();

	int32 OrbitOffsetValue = -1;
	if (LODLevel->OrbitModules.Num() > 0)
	{
		UParticleLODLevel* HighestLODLevel = SpriteTemplate->LODLevels[0];
		check(HighestLODLevel);

		UParticleModuleOrbit* OrbitModule = HighestLODLevel->OrbitModules[LODLevel->OrbitModules.Num() - 1];
		if (OrbitModule)
		{
			uint32* OrbitOffsetIndex = SpriteTemplate->ModuleOffsetMap.Find(OrbitModule);
			if (OrbitOffsetIndex)
			{
				OrbitOffsetValue = *OrbitOffsetIndex;
			}
		}
	}
	return OrbitOffsetValue;
}

FVector FParticleEmitterInstance::GetParticleLocationWithOrbitOffset(FBaseParticle* Particle)
{
	int32 OrbitOffsetValue = GetOrbitPayloadOffset();
	if (OrbitOffsetValue == -1)
	{
		return Particle->Location;
	}
	else
	{
		int32 CurrentOffset = OrbitOffsetValue;
		const uint8* ParticleBase = (const uint8*)Particle;
		PARTICLE_ELEMENT(FOrbitChainModuleInstancePayload, OrbitPayload);
		return Particle->Location + FVector(OrbitPayload.Offset);
	}
}

/**
 *	Update the bounding box for the emitter
 *
 *	@param	DeltaTime		The time slice to use
 */
void FParticleEmitterInstance::UpdateBoundingBox(float DeltaTime)
{
	SCOPE_CYCLE_COUNTER(STAT_ParticleUpdateBounds);
	{
		const UParticleSystem* Template = Component.GetTemplate();
		bool bUpdateBox = ((Component.GetIsWarmingUp() == false) && (Template != NULL) && (Template->bUseFixedRelativeBoundingBox == false));

		// Take component scale into account
		FVector Scale = Component.GetComponentTransform().GetScale3D();

		UParticleLODLevel* LODLevel = GetCurrentLODLevelChecked();

		FVector	NewLocation;
		float	NewRotation;
		if (bUpdateBox)
		{
			ParticleBoundingBox.Init();
		}
		UParticleLODLevel* HighestLODLevel = SpriteTemplate->LODLevels[0];
		check(HighestLODLevel);

		FVector ParticlePivotOffset(-0.5f,-0.5f,0.0f);
		if( bUpdateBox )
		{
			uint32 NumModules = HighestLODLevel->Modules.Num();
			for( uint32 i = 0; i < NumModules; ++i )
			{
				UParticleModulePivotOffset* Module = Cast<UParticleModulePivotOffset>(HighestLODLevel->Modules[i]);
				if( Module )
				{
					FVector2D PivotOff = Module->PivotOffset;
					ParticlePivotOffset += FVector(PivotOff.X, PivotOff.Y, 0.0f);
					break;
				}
			}
		}

		// Store off the orbit offset, if there is one
		int32 OrbitOffsetValue = GetOrbitPayloadOffset();

		// For each particle, offset the box appropriately 
		FVector MinVal(HALF_WORLD_MAX);
		FVector MaxVal(-HALF_WORLD_MAX);
		
		const bool bUseLocalSpace = LODLevel->RequiredModule->bUseLocalSpace;

		const FMatrix ComponentToWorld = bUseLocalSpace 
			? Component.GetComponentToWorld().ToMatrixWithScale() 
			: FMatrix::Identity;

		bool bSkipDoubleSpawnUpdate = !SpriteTemplate->bUseLegacySpawningBehavior;
		for (int32 i=0; i<ActiveParticles; i++)
		{
			DECLARE_PARTICLE(Particle, ParticleData + ParticleStride * ParticleIndices[i]);
			
			// Do linear integrator and update bounding box
			// Do angular integrator, and wrap result to within +/- 2 PI
			Particle.OldLocation	= Particle.Location;

			bool bJustSpawned = (Particle.Flags & STATE_Particle_JustSpawned) != 0;
			Particle.Flags &= ~STATE_Particle_JustSpawned;

			//Don't update position for newly spawned particles. They already have a partial update applied during spawn.
			bool bSkipUpdate = bJustSpawned && bSkipDoubleSpawnUpdate;

			if ((Particle.Flags & STATE_Particle_Freeze) == 0 && !bSkipUpdate)
			{
				if ((Particle.Flags & STATE_Particle_FreezeTranslation) == 0)
				{
					NewLocation = Particle.Location + FVector(DeltaTime * Particle.Velocity);
				}
				else
				{
					NewLocation = Particle.Location;
				}
				if ((Particle.Flags & STATE_Particle_FreezeRotation) == 0)
				{
					NewRotation = (DeltaTime * Particle.RotationRate) + Particle.Rotation;
				}
				else
				{
					NewRotation = Particle.Rotation;
				}
			}
			else
			{
				NewLocation = Particle.Location;
				NewRotation = Particle.Rotation;
			}

			FVector::FReal LocalMax(0.0f);

			if (bUpdateBox)
			{	
				if (OrbitOffsetValue == -1)
				{
					LocalMax = (FVector(Particle.Size) * Scale).GetAbsMax();
				}
				else
				{
					int32 CurrentOffset = OrbitOffsetValue;
					const uint8* ParticleBase = (const uint8*)&Particle;
					PARTICLE_ELEMENT(FOrbitChainModuleInstancePayload, OrbitPayload);
					LocalMax = OrbitPayload.Offset.GetAbsMax();
				}

				LocalMax += (FVector(Particle.Size) * ParticlePivotOffset).GetAbsMax();
			}

			NewLocation			+= PositionOffsetThisTick;
			Particle.OldLocation+= PositionOffsetThisTick;
						
			Particle.Location	 = NewLocation;
			Particle.Rotation	 = FMath::Fmod(NewRotation, 2.f*(float)UE_PI);

			if (bUpdateBox)
			{	
				FVector PositionForBounds = NewLocation;

				if (bUseLocalSpace)
				{
					// Note: building the bounding box in world space as that gives tighter bounds than transforming a local space AABB into world space
					PositionForBounds = ComponentToWorld.TransformPosition(NewLocation);
				}

				// Treat each particle as a cube whose sides are the length of the maximum component
				// This handles the particle's extents changing due to being camera facing
				MinVal[0] = FMath::Min(MinVal[0], PositionForBounds.X - LocalMax);
				MaxVal[0] = FMath::Max(MaxVal[0], PositionForBounds.X + LocalMax);
				MinVal[1] = FMath::Min(MinVal[1], PositionForBounds.Y - LocalMax);
				MaxVal[1] = FMath::Max(MaxVal[1], PositionForBounds.Y + LocalMax);
				MinVal[2] = FMath::Min(MinVal[2], PositionForBounds.Z - LocalMax);
				MaxVal[2] = FMath::Max(MaxVal[2], PositionForBounds.Z + LocalMax);
			}
		}

		if (bUpdateBox)
		{
			ParticleBoundingBox = FBox(MinVal, MaxVal);
		}
	}
}

/**
 * Force the bounding box to be updated.
 */
void FParticleEmitterInstance::ForceUpdateBoundingBox()
{
	{
		// Take component scale into account
		FVector Scale = Component.GetComponentTransform().GetScale3D();

		UParticleLODLevel* LODLevel = GetCurrentLODLevelChecked();
		UParticleLODLevel* HighestLODLevel = SpriteTemplate->LODLevels[0];
		check(HighestLODLevel);

		ParticleBoundingBox.Init();

		// Store off the orbit offset, if there is one
		int32 OrbitOffsetValue = GetOrbitPayloadOffset();

		const bool bUseLocalSpace = LODLevel->RequiredModule->bUseLocalSpace;

		const FMatrix ComponentToWorld = bUseLocalSpace 
			? Component.GetComponentToWorld().ToMatrixWithScale() 
			: FMatrix::Identity;

		// For each particle, offset the box appropriately 
		FVector MinVal(HALF_WORLD_MAX);
		FVector MaxVal(-HALF_WORLD_MAX);
		
		for (int32 i=0; i<ActiveParticles; i++)
		{
			DECLARE_PARTICLE(Particle, ParticleData + ParticleStride * ParticleIndices[i]);

			FVector::FReal LocalMax(0.0f);

			if (OrbitOffsetValue == -1)
			{
				LocalMax = (FVector(Particle.Size) * Scale).GetAbsMax();
			}
			else
			{
				int32 CurrentOffset = OrbitOffsetValue;
				const uint8* ParticleBase = (const uint8*)&Particle;
				PARTICLE_ELEMENT(FOrbitChainModuleInstancePayload, OrbitPayload);
				LocalMax = OrbitPayload.Offset.GetAbsMax();
			}

			FVector PositionForBounds = Particle.Location;

			if (bUseLocalSpace)
			{
				// Note: building the bounding box in world space as that gives tighter bounds than transforming a local space AABB into world space
				PositionForBounds = ComponentToWorld.TransformPosition(Particle.Location);
			}

			// Treat each particle as a cube whose sides are the length of the maximum component
			// This handles the particle's extents changing due to being camera facing
			MinVal[0] = FMath::Min(MinVal[0], PositionForBounds.X - LocalMax);
			MaxVal[0] = FMath::Max(MaxVal[0], PositionForBounds.X + LocalMax);
			MinVal[1] = FMath::Min(MinVal[1], PositionForBounds.Y - LocalMax);
			MaxVal[1] = FMath::Max(MaxVal[1], PositionForBounds.Y + LocalMax);
			MinVal[2] = FMath::Min(MinVal[2], PositionForBounds.Z - LocalMax);
			MaxVal[2] = FMath::Max(MaxVal[2], PositionForBounds.Z + LocalMax);
		}

		ParticleBoundingBox = FBox(MinVal, MaxVal);
	}
}

/**
 *	Retrieved the per-particle bytes that this emitter type requires.
 *
 *	@return	uint32	The number of required bytes for particles in the instance
 */
uint32 FParticleEmitterInstance::RequiredBytes()
{
	// If ANY LOD level has subUV, the size must be taken into account.
	uint32 uiBytes = 0;
	bool bHasSubUV = false;
	for (int32 LODIndex = 0; (LODIndex < SpriteTemplate->LODLevels.Num()) && !bHasSubUV; LODIndex++)
	{
		// This code assumes that the module stacks are identical across LOD levevls...
		UParticleLODLevel* LODLevel = SpriteTemplate->GetLODLevel(LODIndex);
		
		if (LODLevel)
		{
			EParticleSubUVInterpMethod	InterpolationMethod	= (EParticleSubUVInterpMethod)LODLevel->RequiredModule->InterpolationMethod;
			if (LODIndex > 0)
			{
				if ((InterpolationMethod != PSUVIM_None) && (bHasSubUV == false))
				{
					UE_LOGF(LogParticles, Warning, "Emitter w/ mismatched SubUV settings: %ls",
							Component.GetTemplate() ? *(Component.GetTemplate()->GetPathName()) : *(Component.GetPathName()));
				}

				if ((InterpolationMethod == PSUVIM_None) && (bHasSubUV == true))
				{
					UE_LOGF(LogParticles, Warning, "Emitter w/ mismatched SubUV settings: %ls",
						Component.GetTemplate() ? *(Component.GetTemplate()->GetPathName()) : *(Component.GetPathName()));
				}
			}
			// Check for SubUV utilization, and update the required bytes accordingly
			if (InterpolationMethod != PSUVIM_None)
			{
				bHasSubUV = true;
			}
		}
	}

	if (bHasSubUV)
	{
		SubUVDataOffset = PayloadOffset;
		uiBytes	= sizeof(FFullSubUVPayload);
	}

	return uiBytes;
}

uint32 FParticleEmitterInstance::GetModuleDataOffset(UParticleModule* Module)
{
	check(SpriteTemplate);

	uint32* Offset = SpriteTemplate->ModuleOffsetMap.Find(Module);
	return (Offset != nullptr) ? *Offset : 0;
}

uint8* FParticleEmitterInstance::GetModuleInstanceData(UParticleModule* Module)
{
	// If there is instance data present, look up the modules offset
	if (InstanceData)
	{
		uint32* Offset = SpriteTemplate->ModuleInstanceOffsetMap.Find(Module);
		if (Offset)
		{
			check(*Offset < (uint32)InstancePayloadSize);
			return &(InstanceData[*Offset]);
		}
	}
	return NULL;
}

/** Get pointer to emitter instance random seed payload data for a particular module */
FParticleRandomSeedInstancePayload* FParticleEmitterInstance::GetModuleRandomSeedInstanceData(UParticleModule* Module)
{
	// If there is instance data present, look up the modules offset
	if (InstanceData)
	{
		uint32* Offset = SpriteTemplate->ModuleRandomSeedInstanceOffsetMap.Find(Module);
		if (Offset)
		{
			check(*Offset < (uint32)InstancePayloadSize);
			return (FParticleRandomSeedInstancePayload*)&(InstanceData[*Offset]);
		}
	}
	return NULL;
}

/**
 *	Get the pointer to the instance data allocated for type data module.
 *
 *	@return	uint8*		The pointer to the data
 */
uint8* FParticleEmitterInstance::GetTypeDataModuleInstanceData()
{
	if (InstanceData && (TypeDataInstanceOffset != -1))
	{
		return &(InstanceData[TypeDataInstanceOffset]);
	}
	return NULL;
}

/**
 *	Calculate the stride of a single particle for this instance
 *
 *	@param	ParticleSize	The size of the particle
 *
 *	@return	uint32			The stride of the particle
 */
uint32 FParticleEmitterInstance::CalculateParticleStride(uint32 InParticleSize)
{
	return InParticleSize;
}

/**
 *	Reset the burst list information for the instance
 */
void FParticleEmitterInstance::ResetBurstList()
{
	//QUICK_SCOPE_CYCLE_COUNTER(STAT_ResetBurstLists);

	for (int32 BurstIndex = 0; BurstIndex < BurstFired.Num(); BurstIndex++)
	{
		FLODBurstFired& CurrBurstFired = BurstFired[BurstIndex];
		for (int32 FiredIndex = 0; FiredIndex < CurrBurstFired.Fired.Num(); FiredIndex++)
		{
			CurrBurstFired.Fired[FiredIndex] = false;
		}
	}
}

/**
 *	Get the current burst rate offset (delta time is artificially increased to generate bursts)
 *
 *	@param	DeltaTime	The time slice (In/Out)
 *	@param	Burst		The number of particles to burst (Output)
 *
 *	@return	float		The time slice increase to use
 */
float FParticleEmitterInstance::GetCurrentBurstRateOffset(float& DeltaTime, int32& Burst)
{
	float SpawnRateInc = 0.0f;

	// Grab the current LOD level
	UParticleLODLevel* LODLevel	= GetCurrentLODLevelChecked();
	if (LODLevel->SpawnModule->BurstList.Num() > 0)
	{
		FRandomStream& RandomStream = LODLevel->SpawnModule->GetRandomStream({*this});

		// For each burst in the list
		for (int32 BurstIdx = 0; BurstIdx < LODLevel->SpawnModule->BurstList.Num(); BurstIdx++)
		{
			FParticleBurst* BurstEntry = &(LODLevel->SpawnModule->BurstList[BurstIdx]);
			// If it hasn't been fired
			if (BurstEntry && LODLevel->Level < BurstFired.Num())
			{
				FLODBurstFired& LocalBurstFired = BurstFired[LODLevel->Level];
				if (BurstIdx < LocalBurstFired.Fired.Num())
				{
					if (LocalBurstFired.Fired[BurstIdx] == false)
					{
						// If it is time to fire it
						if (EmitterTime >= BurstEntry->Time)
						{
							// Make sure there is a valid time slice
							if (DeltaTime < 0.00001f)
							{
								DeltaTime = 0.00001f;
							}
							// Calculate the increase time slice
							int32 Count = BurstEntry->Count;
							if (BurstEntry->CountLow > -1)
							{
								Count = RandomStream.RandRange(BurstEntry->CountLow, BurstEntry->Count);
							}
							// Take in to account scale.
							float Scale = LODLevel->SpawnModule->BurstScale.GetValue(EmitterTime, Component.GetDistributionData());
							Count = FMath::CeilToInt(Count * Scale);
							SpawnRateInc += Count / DeltaTime;
							Burst += Count;
							LocalBurstFired.Fired[BurstIdx] = true;
						}
					}
				}
			}
		}
	}

	return SpawnRateInc;
}

/**
 *	Reset the particle parameters
 */
void FParticleEmitterInstance::ResetParticleParameters(float DeltaTime)
{
	UParticleLODLevel* LODLevel = GetCurrentLODLevelChecked();
	UParticleLODLevel* HighestLODLevel = SpriteTemplate->LODLevels[0];
	check(HighestLODLevel);

	// Store off any orbit offset values
	TArray<int32, TInlineAllocator<8>> OrbitOffsets;
	int32 OrbitCount = LODLevel->OrbitModules.Num();
	for (int32 OrbitIndex = 0; OrbitIndex < OrbitCount; OrbitIndex++)
	{
		UParticleModuleOrbit* OrbitModule = HighestLODLevel->OrbitModules[OrbitIndex];
		if (OrbitModule)
		{
			uint32* OrbitOffset = SpriteTemplate->ModuleOffsetMap.Find(OrbitModule);
			if (OrbitOffset)
			{
				OrbitOffsets.Add(*OrbitOffset);
			}
		}
	}

	bool bSkipDoubleSpawnUpdate = !SpriteTemplate->bUseLegacySpawningBehavior;
	for (int32 ParticleIndex = 0; ParticleIndex < ActiveParticles; ParticleIndex++)
	{
		DECLARE_PARTICLE(Particle, ParticleData + ParticleStride * ParticleIndices[ParticleIndex]);
		Particle.Velocity		= Particle.BaseVelocity;
		Particle.Size = GetParticleBaseSize(Particle);
		Particle.RotationRate	= Particle.BaseRotationRate;
		Particle.Color = Particle.BaseColor;

		bool bJustSpawned = (Particle.Flags & STATE_Particle_JustSpawned) != 0;

		//Don't update position for newly spawned particles. They already have a partial update applied during spawn.
		bool bSkipUpdate = bJustSpawned && bSkipDoubleSpawnUpdate;

		Particle.RelativeTime	+= bSkipUpdate ? 0.0f : Particle.OneOverMaxLifetime * DeltaTime;

		if (CameraPayloadOffset > 0)
		{
			int32 CurrentOffset = CameraPayloadOffset;
			const uint8* ParticleBase = (const uint8*)&Particle;
			PARTICLE_ELEMENT(FCameraOffsetParticlePayload, CameraOffsetPayload);
			CameraOffsetPayload.Offset = CameraOffsetPayload.BaseOffset;
		}
		for (int32 OrbitIndex = 0; OrbitIndex < OrbitOffsets.Num(); OrbitIndex++)
		{
			int32 CurrentOffset = OrbitOffsets[OrbitIndex];
			const uint8* ParticleBase = (const uint8*)&Particle;
			PARTICLE_ELEMENT(FOrbitChainModuleInstancePayload, OrbitPayload);
			OrbitPayload.PreviousOffset = OrbitPayload.Offset;
			OrbitPayload.Offset = OrbitPayload.BaseOffset;
			OrbitPayload.RotationRate = OrbitPayload.BaseRotationRate;
		}
	}
}

/**
 *	Calculate the orbit offset data.
 */
void FParticleEmitterInstance::CalculateOrbitOffset(FOrbitChainModuleInstancePayload& Payload, 
	FVector& AccumOffset, FVector& AccumRotation, FVector& AccumRotationRate, 
	float DeltaTime, FVector& Result, FMatrix& RotationMat)
{
	AccumRotation += AccumRotationRate * DeltaTime;
	Payload.Rotation = FVector3f(AccumRotation);
	if (AccumRotation.IsNearlyZero() == false)
	{
		FVector RotRot = RotationMat.TransformVector(AccumRotation);
		FVector ScaledRotation = RotRot * 360.0f;
		FRotator Rotator = FRotator::MakeFromEuler(ScaledRotation);
		FMatrix RotMat = FRotationMatrix(Rotator);

		RotationMat *= RotMat;

		Result = RotationMat.TransformPosition(AccumOffset);
	}
	else
	{
		Result = AccumOffset;
	}

	AccumOffset.X = 0.0f;;
	AccumOffset.Y = 0.0f;;
	AccumOffset.Z = 0.0f;;
	AccumRotation.X = 0.0f;;
	AccumRotation.Y = 0.0f;;
	AccumRotation.Z = 0.0f;;
	AccumRotationRate.X = 0.0f;;
	AccumRotationRate.Y = 0.0f;;
	AccumRotationRate.Z = 0.0f;;
}

void FParticleEmitterInstance::UpdateOrbitData(float DeltaTime)
{
	UParticleLODLevel* LODLevel = GetCurrentLODLevelChecked();
	int32 ModuleCount = LODLevel->OrbitModules.Num();
	if (ModuleCount > 0)
	{
		UParticleLODLevel* HighestLODLevel = SpriteTemplate->LODLevels[0];
		check(HighestLODLevel);

		TArray<FVector, TInlineAllocator<8>> Offsets;
		Offsets.AddZeroed(ModuleCount + 1);

		TArray<int32, TInlineAllocator<8>> ModuleOffsets;
		ModuleOffsets.AddZeroed(ModuleCount + 1);
		for (int32 ModOffIndex = 0; ModOffIndex < ModuleCount; ModOffIndex++)
		{
			UParticleModuleOrbit* HighestOrbitModule = HighestLODLevel->OrbitModules[ModOffIndex];
			check(HighestOrbitModule);

			ModuleOffsets[ModOffIndex] = GetModuleDataOffset(HighestOrbitModule);
		}

		for(int32 i=ActiveParticles-1; i>=0; i--)
		{
			int32 OffsetIndex = 0;
			const int32	CurrentIndex	= ParticleIndices[i];
			const uint8* ParticleBase	= ParticleData + CurrentIndex * ParticleStride;
			FBaseParticle& Particle		= *((FBaseParticle*) ParticleBase);
			if ((Particle.Flags & STATE_Particle_Freeze) == 0)
			{
				FVector AccumulatedOffset(0.0f);
				FVector AccumulatedRotation(0.0f);
				FVector AccumulatedRotationRate(0.0f);

				FOrbitChainModuleInstancePayload* LocalOrbitPayload = NULL;
				FOrbitChainModuleInstancePayload* PrevOrbitPayload = NULL;
				uint8 PrevOrbitChainMode = 0;
				FMatrix AccumRotMatrix;
				AccumRotMatrix.SetIdentity();

				int32 CurrentAccumCount = 0;

				for (int32 OrbitIndex = 0; OrbitIndex < ModuleCount; OrbitIndex++)
				{
					int32 CurrentOffset = ModuleOffsets[OrbitIndex];
					UParticleModuleOrbit* OrbitModule = LODLevel->OrbitModules[OrbitIndex];
					check(OrbitModule);

					if (CurrentOffset == 0)
					{
						continue;
					}

					PARTICLE_ELEMENT(FOrbitChainModuleInstancePayload, OrbitPayload);

					// The last orbit module holds the last final offset position
					bool bCalculateOffset = false;
					if (OrbitIndex == (ModuleCount - 1))
					{
						LocalOrbitPayload = &OrbitPayload;
						bCalculateOffset = true;
					}

					// Determine the offset, rotation, rotationrate for the current particle
					if (OrbitModule->ChainMode == EOChainMode_Add)
					{
						if (OrbitModule->bEnabled == true)
						{
							AccumulatedOffset += FVector(OrbitPayload.Offset);
							AccumulatedRotation += FVector(OrbitPayload.Rotation);
							AccumulatedRotationRate += FVector(OrbitPayload.RotationRate);
						}
					}
					else
					if (OrbitModule->ChainMode == EOChainMode_Scale)
					{
						if (OrbitModule->bEnabled == true)
						{
							AccumulatedOffset *= FVector(OrbitPayload.Offset);
							AccumulatedRotation *= FVector(OrbitPayload.Rotation);
							AccumulatedRotationRate *= FVector(OrbitPayload.RotationRate);
						}
					}
					else
					if (OrbitModule->ChainMode == EOChainMode_Link)
					{
						if ((OrbitIndex > 0) && (PrevOrbitChainMode == EOChainMode_Link))
						{
							// Calculate the offset with the current accumulation
							FVector ResultOffset;
							CalculateOrbitOffset(*PrevOrbitPayload, 
								AccumulatedOffset, AccumulatedRotation, AccumulatedRotationRate, 
								DeltaTime, ResultOffset, AccumRotMatrix);
							if (OrbitModule->bEnabled == false)
							{
								AccumulatedOffset = FVector::ZeroVector;
								AccumulatedRotation = FVector::ZeroVector;
								AccumulatedRotationRate = FVector::ZeroVector;
							}
							Offsets[OffsetIndex++] = ResultOffset;
						}

						if (OrbitModule->bEnabled == true)
						{
							AccumulatedOffset = FVector(OrbitPayload.Offset);
							AccumulatedRotation = FVector(OrbitPayload.Rotation);
							AccumulatedRotationRate = FVector(OrbitPayload.RotationRate);
						}
					}

					if (bCalculateOffset == true)
					{
						// Push the current offset into the array
						FVector ResultOffset;
						CalculateOrbitOffset(OrbitPayload, 
							AccumulatedOffset, AccumulatedRotation, AccumulatedRotationRate, 
							DeltaTime, ResultOffset, AccumRotMatrix);
						Offsets[OffsetIndex++] = ResultOffset;
					}

					if (OrbitModule->bEnabled)
					{
						PrevOrbitPayload = &OrbitPayload;
						PrevOrbitChainMode = OrbitModule->ChainMode;
					}
				}

				if (LocalOrbitPayload != NULL)
				{
					LocalOrbitPayload->Offset = FVector3f::ZeroVector;
					for (int32 AccumIndex = 0; AccumIndex < OffsetIndex; AccumIndex++)
					{
						LocalOrbitPayload->Offset += FVector3f(Offsets[AccumIndex]);
					}

					FMemory::Memzero(Offsets.GetData(), sizeof(FVector) * (ModuleCount + 1));
				}
			}
		}
	}
}

void FParticleEmitterInstance::ParticlePrefetch()
{
	for (int32 ParticleIndex = 0; ParticleIndex < ActiveParticles; ParticleIndex++)
	{
		PARTICLE_INSTANCE_PREFETCH(this, ParticleIndex);
	}
}

void FParticleEmitterInstance::CheckSpawnCount(int32 InNewCount, int32 InMaxCount)
{
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	{
		if (Component.HasWorldSettings())
		{
			const int32 SizeScalar = sizeof(FParticleSpriteVertex);

			int32 MyIndex = -1;
			TArrayView<FParticleEmitterInstance*> EmitterInstances = Component.GetEmitterInstances();
			for (int32 CheckIdx = 0; CheckIdx < EmitterInstances.Num(); CheckIdx++)
			{
				if (EmitterInstances[CheckIdx] == this)
				{
					MyIndex = CheckIdx;
					break;
				}
			}

			FString ErrorMessage = 
				FString::Printf(TEXT("Emitter %2d spawn vertices: %10d (%8.3f kB of verts), clamp to %10d (%8.3f kB) - spawned %4d: %s"),
				MyIndex,
				InNewCount, 
				(float)(InNewCount * 4 * SizeScalar) / 1024.0f,
				InMaxCount, 
				(float)(InMaxCount * 4 * SizeScalar) / 1024.0f,
				InNewCount - ActiveParticles,
				Component.GetTemplate() ? *Component.GetTemplate()->GetPathName() : TEXT("No template"));
			FColor ErrorColor(255,255,0);
			if (GEngine->OnScreenDebugMessageExists((uint64)(0x8000000 | (PTRINT)this)) == false)
			{
				UE_LOGF(LogParticles, Log, "%ls", *ErrorMessage);
			}
			GEngine->AddOnScreenDebugMessage((uint64)(0x8000000 | (PTRINT)this), 5.0f, ErrorColor,ErrorMessage);
		}
	}
#endif	//#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
}

/**
 *	Spawn particles for this emitter instance
 *
 *	@param	DeltaTime		The time slice to spawn over
 *
 *	@return	float			The leftover fraction of spawning
 */
float FParticleEmitterInstance::Spawn(float DeltaTime)
{
	SCOPE_CYCLE_COUNTER(STAT_SpriteSpawnTime);
	UParticleLODLevel* LODLevel = GetCurrentLODLevelChecked();

	// For beams, we probably want to ignore the SpawnRate distribution,
	// and focus strictly on the BurstList...
	float SpawnRate = 0.0f;
	int32 SpawnCount = 0;
	int32 BurstCount = 0;
	float SpawnRateDivisor = 0.0f;
	float OldLeftover = SpawnFraction;

	UParticleLODLevel* HighestLODLevel = SpriteTemplate->LODLevels[0];

	bool bProcessSpawnRate = true;
	bool bProcessBurstList = true;
	//int32 DetailMode = Component.GetCurrentDetailMode();

	if (SpriteTemplate->QualityLevelSpawnRateScale > 0.0f)
	{
		// Process all Spawning modules that are present in the emitter.
		for (int32 SpawnModIndex = 0; SpawnModIndex < LODLevel->SpawningModules.Num(); SpawnModIndex++)
		{
			UParticleModuleSpawnBase* SpawnModule = LODLevel->SpawningModules[SpawnModIndex];
			if (SpawnModule && SpawnModule->bEnabled)
			{
				UParticleModule* OffsetModule = HighestLODLevel->SpawningModules[SpawnModIndex];
				uint32 Offset = GetModuleDataOffset(OffsetModule);

				// Update the spawn rate
				int32 Number = 0;
				float Rate = 0.0f;
				if (SpawnModule->GetSpawnAmount({*this}, Offset, OldLeftover, DeltaTime, Number, Rate) == false)
				{
					bProcessSpawnRate = false;
				}

				Number = FMath::Max<int32>(0, Number);
				Rate = FMath::Max<float>(0.0f, Rate);

				SpawnCount += Number;
				SpawnRate += Rate;
				// Update the burst list
				int32 BurstNumber = 0;
				if (SpawnModule->GetBurstCount(this, Offset, OldLeftover, DeltaTime, BurstNumber) == false)
				{
					bProcessBurstList = false;
				}

				BurstCount += BurstNumber;
			}
		}

		// Figure out spawn rate for this tick.
		if (bProcessSpawnRate)
		{
			float RateScale = LODLevel->SpawnModule->RateScale.GetValue(EmitterTime, Component.GetDistributionData()) * LODLevel->SpawnModule->GetGlobalRateScale();
			SpawnRate += LODLevel->SpawnModule->Rate.GetValue(EmitterTime, Component.GetDistributionData()) * RateScale;
			SpawnRate = FMath::Max<float>(0.0f, SpawnRate);
		}

		// Take Bursts into account as well...
		if (bProcessBurstList)
		{
			int32 Burst = 0;
			float BurstTime = GetCurrentBurstRateOffset(DeltaTime, Burst);
			BurstCount += Burst;
		}

		float QualityMult = SpriteTemplate->GetQualityLevelSpawnRateMult();
		SpawnRate = FMath::Max<float>(0.0f, SpawnRate * QualityMult);
		BurstCount = FMath::CeilToInt(BurstCount * QualityMult);
	}
	else
	{
		// Disable any spawning if MediumDetailSpawnRateScale is 0 and we are not in high detail mode
		SpawnRate = 0.0f;
		SpawnCount = 0;
		BurstCount = 0;
	}

	// Spawn new particles...
	if ((SpawnRate > 0.f) || (BurstCount > 0))
	{
		float SafetyLeftover = OldLeftover;
		// Ensure continuous spawning... lots of fiddling.
		float	NewLeftover = OldLeftover + DeltaTime * SpawnRate;
		int32		Number		= FMath::FloorToInt(NewLeftover);
		float	Increment	= (SpawnRate > 0.0f) ? (1.f / SpawnRate) : 0.0f;
		float	StartTime = DeltaTime + OldLeftover * Increment - Increment;
		NewLeftover			= NewLeftover - Number;

		// Handle growing arrays.
		bool bProcessSpawn = true;
		int32 NewCount = ActiveParticles + Number + BurstCount;

		if (NewCount > FXConsoleVariables::MaxCPUParticlesPerEmitter)
		{
			int32 MaxNewParticles = FXConsoleVariables::MaxCPUParticlesPerEmitter - ActiveParticles;
			BurstCount = FMath::Min(MaxNewParticles, BurstCount);
			MaxNewParticles -= BurstCount;
			Number = FMath::Min(MaxNewParticles, Number);
			NewCount = ActiveParticles + Number + BurstCount;
		}

		float	BurstIncrement = SpriteTemplate->bUseLegacySpawningBehavior ? (BurstCount > 0.0f) ? (1.f / BurstCount) : 0.0f : 0.0f;
		float	BurstStartTime = SpriteTemplate->bUseLegacySpawningBehavior ? DeltaTime * BurstIncrement : 0.0f;

		if (NewCount >= MaxActiveParticles)
		{
			if (DeltaTime < PeakActiveParticleUpdateDelta)
			{
				bProcessSpawn = Resize(NewCount + FMath::TruncToInt(FMath::Sqrt(FMath::Sqrt((float)NewCount)) + 1));
			}
			else
			{
				bProcessSpawn = Resize((NewCount + FMath::TruncToInt(FMath::Sqrt(FMath::Sqrt((float)NewCount)) + 1)), false);
			}
		}

		if (bProcessSpawn == true)
		{
			FParticleEventInstancePayload* EventPayload = NULL;
			if (LODLevel->EventGenerator)
			{
				EventPayload = (FParticleEventInstancePayload*)GetModuleInstanceData(LODLevel->EventGenerator);
				if (EventPayload && !EventPayload->bSpawnEventsPresent && !EventPayload->bBurstEventsPresent)
				{
					EventPayload = NULL;
				}
			}

			const FVector InitialLocation = EmitterToSimulation.GetOrigin();

			// Spawn particles.
			SpawnParticles( Number, StartTime, Increment, InitialLocation, FVector::ZeroVector, EventPayload );

			// Burst particles.
			SpawnParticles(BurstCount, BurstStartTime, BurstIncrement, InitialLocation, FVector::ZeroVector, EventPayload);

			return NewLeftover;
		}
		return SafetyLeftover;
	}

	return SpawnFraction;
}

/**
* Fixup particle indices to only have valid entries.
*/
void FParticleEmitterInstance::FixupParticleIndices()
{
	// Something is wrong and particle data are be invalid. Try to fix-up things.
	TBitArray<> UsedIndices(false, MaxActiveParticles);

	for (int32 i = 0; i < ActiveParticles; ++i)
	{
		const uint16 UsedIndex = ParticleIndices[i];
		if (UsedIndex < MaxActiveParticles && UsedIndices[UsedIndex] == 0)
		{
			UsedIndices[UsedIndex] = 1;
		}
		else
		{
			if (i != ActiveParticles - 1)
			{
				// Remove this bad or duplicated index
				ParticleIndices[i] = ParticleIndices[ActiveParticles - 1];
			}
			// Decrease particle count.
			--ActiveParticles;

			// Retry the new index.
			--i;
		}
	}

	for (int32 i = ActiveParticles; i < MaxActiveParticles; ++i)
	{
		const int32 FreeIndex = UsedIndices.FindAndSetFirstZeroBit();
		if (ensure(FreeIndex != INDEX_NONE))
		{
			ParticleIndices[i] =  (uint16)FreeIndex;
		}
		else // Can't really handle that.
		{
			ParticleIndices[i] = (uint16)i;
		}
	}
}
/**
 * Spawn the indicated number of particles.
 *
 * @param Count The number of particles to spawn.
 * @param StartTime			The local emitter time at which to begin spawning particles.
 * @param Increment			The time delta between spawned particles.
 * @param InitialLocation	The initial location of spawned particles.
 * @param InitialVelocity	The initial velocity of spawned particles.
 * @param EventPayload		Event generator payload if events should be triggered.
 */
void FParticleEmitterInstance::SpawnParticles( int32 Count, float StartTime, float Increment, const FVector& InitialLocation, const FVector& InitialVelocity, struct FParticleEventInstancePayload* EventPayload )
{
	UParticleLODLevel* LODLevel = GetCurrentLODLevelChecked();

	check(ActiveParticles <= MaxActiveParticles);
	check(LODLevel->EventGenerator != NULL || EventPayload == NULL);

	// Ensure we don't access particle beyond what is allocated.
	ensure( ActiveParticles + Count <= MaxActiveParticles );
	Count = FMath::Min<int32>(Count, MaxActiveParticles - ActiveParticles);

	if (EventPayload && EventPayload->bBurstEventsPresent && Count > 0)
	{
		LODLevel->EventGenerator->HandleParticleBurst(this, EventPayload, Count);
	}
	
	auto SpawnInternal = [&](bool bLegacySpawnBehavior)
	{
		UParticleLODLevel* HighestLODLevel = SpriteTemplate->LODLevels[0];
		float SpawnTime = StartTime;
		float Interp = 1.0f;
		const float InterpIncrement = (Count > 0 && Increment > 0.0f) ? (1.0f / (float)Count) : 0.0f;
		for (int32 i = 0; i < Count; i++)
		{
			// Workaround to released data.
			if (!ensure(ParticleData && ParticleIndices))
			{
				static bool bErrorReported = false;
				if (!bErrorReported)
				{
					UE_LOGF(LogParticles, Error, "Detected null particles. Template : %ls, Component %ls", *GetNameSafe(Component.IsValidLowLevel() ? Component.GetTemplate() : nullptr), *Component.GetFullName());
					bErrorReported = true;
				}
				ActiveParticles = 0;
				Count = 0;
				continue;
			}

			// Workaround to corrupted indices.
			uint16 NextFreeIndex = ParticleIndices[ActiveParticles];
			if (!ensure(NextFreeIndex < MaxActiveParticles))
			{
				UE_LOGF(LogParticles, Error, "Detected corrupted particle indices. Template : %ls, Component %ls", *GetNameSafe(Component.IsValidLowLevel() ? Component.GetTemplate() : nullptr), *Component.GetFullName());
				FixupParticleIndices();
				NextFreeIndex = ParticleIndices[ActiveParticles];
			}

			DECLARE_PARTICLE_PTR(Particle, ParticleData + ParticleStride * NextFreeIndex);
			const uint32 CurrentParticleIndex = ActiveParticles++;

			if (bLegacySpawnBehavior)
			{
				StartTime -= Increment;
				Interp -= InterpIncrement;
			}

			PreSpawn(Particle, InitialLocation, InitialVelocity);
			for (int32 ModuleIndex = 0; ModuleIndex < LODLevel->SpawnModules.Num(); ModuleIndex++)
			{
				UParticleModule* SpawnModule = LODLevel->SpawnModules[ModuleIndex];
				if (SpawnModule->bEnabled)
				{
					UParticleModule* OffsetModule = HighestLODLevel->SpawnModules[ModuleIndex];
					SpawnModule->Spawn({ *this, (int32)GetModuleDataOffset(OffsetModule), SpawnTime, Particle });

					ensureMsgf(!Particle->Location.ContainsNaN(), TEXT("NaN in Particle Location. Template: %s, Component: %s"), *GetNameSafe(Component.GetTemplate()), *Component.GetPathName());
				}
			}
			PostSpawn(Particle, Interp, SpawnTime);

			// Spawn modules may set a relative time greater than 1.0f to indicate that a particle should not be spawned. We kill these particles.
			if (Particle->RelativeTime > 1.0f)
			{
				KillParticle(CurrentParticleIndex);

				// Process next particle
				continue;
			}
			
			if (EventPayload)
			{
				if (EventPayload->bSpawnEventsPresent)
				{
					LODLevel->EventGenerator->HandleParticleSpawned(this, EventPayload, Particle);
				}
			}

			if (!bLegacySpawnBehavior)
			{
				SpawnTime -= Increment;
				Interp -= InterpIncrement;
			}

			INC_DWORD_STAT(STAT_SpriteParticlesSpawned);
		}
	};

	if (SpriteTemplate->bUseLegacySpawningBehavior)
	{
		SpawnInternal(true);
	}
	else
	{
		SpawnInternal(false);
	}
}

UParticleLODLevel* FParticleEmitterInstance::GetCurrentLODLevelChecked()
{
	check(SpriteTemplate != NULL);
	UParticleLODLevel* LODLevel = SpriteTemplate->GetCurrentLODLevel(this);
	check(LODLevel != NULL);
	check(LODLevel->RequiredModule != NULL);
	return LODLevel;
}

/**
 *	Spawn/burst the given particles...
 *
 *	@param	DeltaTime		The time slice to spawn over.
 *	@param	InSpawnCount	The number of particles to forcibly spawn.
 *	@param	InBurstCount	The number of particles to forcibly burst.
 *	@param	InLocation		The location to spawn at.
 *	@param	InVelocity		OPTIONAL velocity to have the particle inherit.
 *
 */
void FParticleEmitterInstance::ForceSpawn(float DeltaTime, int32 InSpawnCount, int32 InBurstCount, 
	FVector& InLocation, FVector& InVelocity)
{
	UParticleLODLevel* LODLevel = GetCurrentLODLevelChecked();

	// For beams, we probably want to ignore the SpawnRate distribution,
	// and focus strictly on the BurstList...
	int32 SpawnCount = InSpawnCount;
	int32 BurstCount = InBurstCount;
	float SpawnRateDivisor = 0.0f;
	float OldLeftover = 0.0f;

	UParticleLODLevel* HighestLODLevel = SpriteTemplate->LODLevels[0];

	bool bProcessSpawnRate = true;
	bool bProcessBurstList = true;

	// Spawn new particles...
	if ((SpawnCount > 0) || (BurstCount > 0))
	{
		int32		Number		= SpawnCount;
		float	Increment	= (SpawnCount > 0) ? (DeltaTime / SpawnCount) : 0;
		float	StartTime	= DeltaTime;
		
		// Handle growing arrays.
		bool bProcessSpawn = true;
		int32 NewCount = ActiveParticles + Number + BurstCount;
		if (NewCount >= MaxActiveParticles)
		{
			if (DeltaTime < PeakActiveParticleUpdateDelta)
			{
				bProcessSpawn = Resize(NewCount + FMath::TruncToInt(FMath::Sqrt(FMath::Sqrt((float)NewCount)) + 1));
			}
			else
			{
				bProcessSpawn = Resize((NewCount + FMath::TruncToInt(FMath::Sqrt(FMath::Sqrt((float)NewCount)) + 1)), false);
			}
		}

		if (bProcessSpawn == true)
		{
			// This logic matches the existing behavior. However, I think the
			// interface for ForceSpawn should treat these values as being in
			// world space and transform them to emitter local space if necessary.
			const bool bUseLocalSpace = LODLevel->RequiredModule->bUseLocalSpace;
			FVector SpawnLocation = bUseLocalSpace ? FVector::ZeroVector : InLocation;
			FVector SpawnVelocity = bUseLocalSpace ? FVector::ZeroVector : InVelocity;

			// Spawn particles.
			SpawnParticles( Number, StartTime, Increment, InLocation, InVelocity, NULL /*EventPayload*/ );

			// Burst particles.
			SpawnParticles( BurstCount, StartTime, 0.0f, InLocation, InVelocity, NULL /*EventPayload*/ );
		}
	}
}

/**
 * Handle any pre-spawning actions required for particles
 *
 * @param Particle			The particle being spawned.
 * @param InitialLocation	The initial location of the particle.
 * @param InitialVelocity	The initial velocity of the particle.
 */
void FParticleEmitterInstance::PreSpawn(FBaseParticle* Particle, const FVector& InitialLocation, const FVector& InitialVelocity)
{
	check(Particle);
	// This isn't a problem w/ the FMemory::Memzero call - it's a problem in general!
	check(ParticleSize > 0);

	// By default, just clear out the particle
	FMemory::Memzero(Particle, ParticleSize);

	// Initialize the particle location.
	Particle->Location = InitialLocation;
	Particle->BaseVelocity = FVector3f(InitialVelocity);
	Particle->Velocity = FVector3f(InitialVelocity);

	// New particles has already updated spawn location
	// Subtract offset here, so deferred location offset in UpdateBoundingBox will return this particle back
	Particle->Location-= PositionOffsetThisTick;
}

/**
 *	Has the instance completed it's run?
 *
 *	@return	bool	true if the instance is completed, false if not
 */
bool FParticleEmitterInstance::HasCompleted()
{
	// Validity check
	if (SpriteTemplate == NULL)
	{
		return true;
	}

	// If it hasn't finished looping or if it loops forever, not completed.
	UParticleLODLevel* LODLevel = GetCurrentLODLevelChecked();
	if ((LODLevel->RequiredModule->EmitterLoops == 0) || 
		(SecondsSinceCreation < (EmitterDuration * LODLevel->RequiredModule->EmitterLoops)))
	{
		return false;
	}

	// If there are active particles, not completed
	if (ActiveParticles > 0)
	{
		return false;
	}

	return true;
}

/**
 *	Handle any post-spawning actions required by the instance
 *
 *	@param	Particle					The particle that was spawned
 *	@param	InterpolationPercentage		The percentage of the time slice it was spawned at
 *	@param	SpawnTIme					The time it was spawned at
 */
void FParticleEmitterInstance::PostSpawn(FBaseParticle* Particle, float InterpolationPercentage, float SpawnTime)
{
	// Interpolate position if using world space.
	UParticleLODLevel* LODLevel = GetCurrentLODLevelChecked();
	if (LODLevel->RequiredModule->bUseLocalSpace == false)
	{
		if (FVector::DistSquared(OldLocation, Location) > 1.f)
		{
			Particle->Location += InterpolationPercentage * (OldLocation - Location);	
		}
	}

	// Offset caused by any velocity
	Particle->OldLocation = Particle->Location;
	Particle->Location   += SpawnTime * FVector(Particle->Velocity);

	// Store a sequence counter.
	Particle->Flags |= ((ParticleCounter++) & STATE_CounterMask);
	Particle->Flags |= STATE_Particle_JustSpawned;
}

/**
 *	Kill off any dead particles. (Remove them from the active array)
 */
void FParticleEmitterInstance::KillParticles()
{
	if (ActiveParticles > 0)
	{
		UParticleLODLevel* LODLevel = GetCurrentLODLevelChecked();
		FParticleEventInstancePayload* EventPayload = NULL;
		if (LODLevel->EventGenerator)
		{
			EventPayload = (FParticleEventInstancePayload*)GetModuleInstanceData(LODLevel->EventGenerator);
			if (EventPayload && (EventPayload->bDeathEventsPresent == false))
			{
				EventPayload = NULL;
			}
		}

		bool bFoundCorruptIndices = false;
		// Loop over the active particles... If their RelativeTime is > 1.0f (indicating they are dead),
		// move them to the 'end' of the active particle list.
		for (int32 i=ActiveParticles-1; i>=0; i--)
		{
			const int32	CurrentIndex	= ParticleIndices[i];
			if (ensure(CurrentIndex < MaxActiveParticles))
			{ 
				const uint8* ParticleBase = ParticleData + CurrentIndex * ParticleStride;
				FBaseParticle& Particle = *((FBaseParticle*)ParticleBase);

				if (Particle.RelativeTime > 1.0f)
				{
					if (EventPayload)
					{
						LODLevel->EventGenerator->HandleParticleKilled(this, EventPayload, &Particle);
					}
					// Move it to the 'back' of the list
					ParticleIndices[i] = ParticleIndices[ActiveParticles-1];
					ParticleIndices[ActiveParticles-1]	= CurrentIndex;
					ActiveParticles--;

					INC_DWORD_STAT(STAT_SpriteParticlesKilled);
				}
			}
			else
			{
				bFoundCorruptIndices = true;
			}
		}

		if (bFoundCorruptIndices)
		{
			UE_LOGF(LogParticles, Error, "Detected corrupted particle indices. Template : %ls, Component %ls", *GetNameSafe(Component.IsValidLowLevel() ? Component.GetTemplate() : nullptr), *Component.GetFullName());			
			FixupParticleIndices();
		}
	}
}

/**
 *	Kill the particle at the given instance
 *
 *	@param	Index		The index of the particle to kill.
 */
void FParticleEmitterInstance::KillParticle(int32 Index)
{
	if (Index < ActiveParticles)
	{
		UParticleLODLevel* LODLevel = GetCurrentLODLevelChecked();
		FParticleEventInstancePayload* EventPayload = NULL;
		if (LODLevel->EventGenerator)
		{
			EventPayload = (FParticleEventInstancePayload*)GetModuleInstanceData(LODLevel->EventGenerator);
			if (EventPayload && (EventPayload->bDeathEventsPresent == false))
			{
				EventPayload = NULL;
			}
		}

		int32 KillIndex = ParticleIndices[Index];

		// Handle the kill event, if needed
		if (EventPayload)
		{
			const uint8* ParticleBase	= ParticleData + KillIndex * ParticleStride;
			FBaseParticle& Particle		= *((FBaseParticle*) ParticleBase);
			LODLevel->EventGenerator->HandleParticleKilled(this, EventPayload, &Particle);
		}

		// Move it to the 'back' of the list
		for (int32 i=Index; i < ActiveParticles - 1; i++)
		{
			ParticleIndices[i] = ParticleIndices[i+1];
		}
		ParticleIndices[ActiveParticles-1] = KillIndex;
		ActiveParticles--;

		INC_DWORD_STAT(STAT_SpriteParticlesKilled);
	}
}

void FParticleEmitterInstance::FakeBursts()
{
	UParticleLODLevel* LODLevel = GetCurrentLODLevelChecked();
	if (LODLevel->SpawnModule->BurstList.Num() > 0)
	{
		// For each burst in the list
		for (int32 BurstIdx = 0; BurstIdx < LODLevel->SpawnModule->BurstList.Num(); BurstIdx++)
		{
			FParticleBurst* BurstEntry = &(LODLevel->SpawnModule->BurstList[BurstIdx]);
			// If it hasn't been fired
			if (LODLevel->Level < BurstFired.Num())
			{
				FLODBurstFired& LocalBurstFired = BurstFired[LODLevel->Level];
				if (BurstIdx < LocalBurstFired.Fired.Num())
				{
					if (EmitterTime >= BurstEntry->Time)
					{
						LocalBurstFired.Fired[BurstIdx] = true;
					}
				}
			}
		}
	}
}

/**
 *	This is used to force "kill" particles irrespective of their duration.
 *	Basically, this takes all particles and moves them to the 'end' of the 
 *	particle list so we can insta kill off trailed particles in the level.
 */
void FParticleEmitterInstance::KillParticlesForced(bool bFireEvents)
{
	UParticleLODLevel* LODLevel = GetCurrentLODLevelChecked();
	FParticleEventInstancePayload* EventPayload = NULL;
	if (bFireEvents == true)
	{
		if (LODLevel->EventGenerator)
		{
			EventPayload = (FParticleEventInstancePayload*)GetModuleInstanceData(LODLevel->EventGenerator);
			if (EventPayload && (EventPayload->bDeathEventsPresent == false))
			{
				EventPayload = NULL;
			}
		}
	}

	// Loop over the active particles and kill them.
	// Move them to the 'end' of the active particle list.
	for (int32 KillIdx = ActiveParticles - 1; KillIdx >= 0; KillIdx--)
	{
		const int32 CurrentIndex = ParticleIndices[KillIdx];
		// Handle the kill event, if needed
		if (EventPayload)
		{
			const uint8* ParticleBase = ParticleData + CurrentIndex * ParticleStride;
			FBaseParticle& Particle = *((FBaseParticle*) ParticleBase);
			LODLevel->EventGenerator->HandleParticleKilled(this, EventPayload, &Particle);
		}
		ParticleIndices[KillIdx] = ParticleIndices[ActiveParticles - 1];
		ParticleIndices[ActiveParticles - 1] = CurrentIndex;
		ActiveParticles--;

		INC_DWORD_STAT(STAT_SpriteParticlesKilled);
	}

	ParticleCounter = 0;
}

/**
 *	Retrieve the particle at the given index
 *
 *	@param	Index			The index of the particle of interest
 *
 *	@return	FBaseParticle*	The pointer to the particle. NULL if not present/active
 */
FBaseParticle* FParticleEmitterInstance::GetParticle(int32 Index)
{
	// See if the index is valid. If not, return NULL
	if ((Index >= ActiveParticles) || (Index < 0))
	{
		return NULL;
	}

	// Grab and return the particle
	DECLARE_PARTICLE_PTR(Particle, ParticleData + ParticleStride * ParticleIndices[Index]);
	return Particle;
}

FBaseParticle* FParticleEmitterInstance::GetParticleDirect(int32 InDirectIndex)
{
	if ((ActiveParticles > 0) && (InDirectIndex < MaxActiveParticles))
	{
		DECLARE_PARTICLE_PTR(Particle, ParticleData + ParticleStride * InDirectIndex);
		return Particle;
	}
	return NULL;
}

/**
 *	Calculates the emitter duration for the instance.
 */
void FParticleEmitterInstance::SetupEmitterDuration()
{
	// Validity check
	if (SpriteTemplate == NULL)
	{
		return;
	}

	// Set up the array for each LOD level
	int32 EDCount = EmitterDurations.Num();
	if ((EDCount == 0) || (EDCount != SpriteTemplate->LODLevels.Num()))
	{
		EmitterDurations.Empty();
		EmitterDurations.InsertUninitialized(0, SpriteTemplate->LODLevels.Num());
	}

	// Calculate the duration for each LOD level
	for (int32 LODIndex = 0; LODIndex < SpriteTemplate->LODLevels.Num(); LODIndex++)
	{
		UParticleLODLevel* TempLOD = SpriteTemplate->LODLevels[LODIndex];
		UParticleModuleRequired* RequiredModule = TempLOD->RequiredModule;

		FRandomStream& RandomStream = RequiredModule->GetRandomStream({*this});

		CurrentDelay = RequiredModule->EmitterDelay + Component.GetEmitterDelay();
		if (RequiredModule->bEmitterDelayUseRange)
		{
			const float	Rand	= RandomStream.FRand();
			CurrentDelay	    = RequiredModule->EmitterDelayLow + 
				((RequiredModule->EmitterDelay - RequiredModule->EmitterDelayLow) * Rand) + Component.GetEmitterDelay();
		}


		if (RequiredModule->bEmitterDurationUseRange)
		{
			const float	Rand		= RandomStream.FRand();
			const float	Duration	= RequiredModule->EmitterDurationLow + 
				((RequiredModule->EmitterDuration - RequiredModule->EmitterDurationLow) * Rand);
			EmitterDurations[TempLOD->Level] = Duration + CurrentDelay;
		}
		else
		{
			EmitterDurations[TempLOD->Level] = RequiredModule->EmitterDuration + CurrentDelay;
		}

		if ((LoopCount == 1) && (RequiredModule->bDelayFirstLoopOnly == true) && 
			((RequiredModule->EmitterLoops == 0) || (RequiredModule->EmitterLoops > 1)))
		{
			EmitterDurations[TempLOD->Level] -= CurrentDelay;
		}
	}

	// Set the current duration
	EmitterDuration	= EmitterDurations[CurrentLODLevelIndex];
}

/**
 *	Checks some common values for GetDynamicData validity
 *
 *	@return	bool		true if GetDynamicData should continue, false if it should return NULL
 */
bool FParticleEmitterInstance::IsDynamicDataRequired(UParticleLODLevel* InCurrentLODLevel)
{
	if ((ActiveParticles <= 0) || 
		(SpriteTemplate && (SpriteTemplate->EmitterRenderMode == ERM_None)))
	{
		return false;
	}

	if ((InCurrentLODLevel == NULL) || (InCurrentLODLevel->bEnabled == false) ||
		((InCurrentLODLevel->RequiredModule->bUseMaxDrawCount == true) && (InCurrentLODLevel->RequiredModule->MaxDrawCount == 0)))
	{
		return false;
	}

	return true;
}

/**
 *	Process received events.
 */
void FParticleEmitterInstance::ProcessParticleEvents(float DeltaTime, bool bSuppressSpawning)
{
	UParticleLODLevel* LODLevel = GetCurrentLODLevelChecked();
	if (LODLevel->EventReceiverModules.Num() > 0)
	{
		TArrayView<FParticleEventSpawnData> SpawnEvents = Component.GetSpawnEvents();
		TArrayView<FParticleEventDeathData> DeathEvents = Component.GetDeathEvents();
		TArrayView<FParticleEventCollideData> CollisionEvents = Component.GetCollisionEvents();
		TArrayView<FParticleEventBurstData> BurstEvents = Component.GetBurstEvents();
		TArrayView<FParticleEventKismetData> KismetEvents = Component.GetKismetEvents();

		for (int32 EventModIndex = 0; EventModIndex < LODLevel->EventReceiverModules.Num(); EventModIndex++)
		{
			int32 EventIndex;
			UParticleModuleEventReceiverBase* EventRcvr = LODLevel->EventReceiverModules[EventModIndex];
			check(EventRcvr);

			if (EventRcvr->WillProcessParticleEvent(EPET_Spawn) && (SpawnEvents.Num() > 0))
			{
				for (EventIndex = 0; EventIndex < SpawnEvents.Num(); EventIndex++)
				{
					EventRcvr->ProcessParticleEvent(this, SpawnEvents[EventIndex], DeltaTime);
				}
			}

			if (EventRcvr->WillProcessParticleEvent(EPET_Death) && (DeathEvents.Num() > 0))
			{
				for (EventIndex = 0; EventIndex < DeathEvents.Num(); EventIndex++)
				{
					EventRcvr->ProcessParticleEvent(this, DeathEvents[EventIndex], DeltaTime);
				}
			}

			if (EventRcvr->WillProcessParticleEvent(EPET_Collision) && (CollisionEvents.Num() > 0))
			{
				for (EventIndex = 0; EventIndex < CollisionEvents.Num(); EventIndex++)
				{
					EventRcvr->ProcessParticleEvent(this, CollisionEvents[EventIndex], DeltaTime);
				}
			}

			if (EventRcvr->WillProcessParticleEvent(EPET_Burst) && (BurstEvents.Num() > 0))
			{
				for (EventIndex = 0; EventIndex < BurstEvents.Num(); EventIndex++)
				{
					EventRcvr->ProcessParticleEvent(this, BurstEvents[EventIndex], DeltaTime);
				}
			}

			if (EventRcvr->WillProcessParticleEvent(EPET_Blueprint) && (KismetEvents.Num() > 0))
			{
				for (EventIndex = 0; EventIndex < KismetEvents.Num(); EventIndex++)
				{
					EventRcvr->ProcessParticleEvent(this, KismetEvents[EventIndex], DeltaTime);
				}
			}
		}
	}
}


/**
 * Captures dynamic replay data for this particle system.
 *
 * @param	OutData		[Out] Data will be copied here
 *
 * @return Returns true if successful
 */
bool FParticleEmitterInstance::FillReplayData( FDynamicEmitterReplayDataBase& OutData )
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_ParticleEmitterInstance_FillReplayData);

	// NOTE: This the base class implementation that should ONLY be called by derived classes' FillReplayData()!

	// Make sure there is a template present
	if (!SpriteTemplate)
	{
		return false;
	}

	// Allocate it for now, but we will want to change this to do some form
	// of caching
	if (ActiveParticles <= 0 || !bEnabled)
	{
		return false;
	}
	// If the template is disabled, don't return data.
	UParticleLODLevel* LODLevel = SpriteTemplate->GetCurrentLODLevel(this);
	if ((LODLevel == NULL) || (LODLevel->bEnabled == false))
	{
		return false;
	}

	// Make sure we will not be allocating enough memory
	check(MaxActiveParticles >= ActiveParticles);

	// Must be filled in by implementation in derived class
	OutData.eEmitterType = DET_Unknown;

	OutData.ActiveParticleCount = ActiveParticles;
	OutData.ParticleStride = ParticleStride;
	OutData.SortMode = SortMode;

	// Take scale into account
	OutData.Scale = FVector3f::OneVector;
	OutData.Scale = FVector3f(Component.GetComponentTransform().GetScale3D());

	int32 ParticleMemSize = MaxActiveParticles * ParticleStride;

	// Allocate particle memory

	OutData.DataContainer.Alloc(ParticleMemSize, MaxActiveParticles);
	INC_DWORD_STAT_BY(STAT_RTParticleData, OutData.DataContainer.MemBlockSize);

	FMemory::BigBlockMemcpy(OutData.DataContainer.ParticleData, ParticleData, ParticleMemSize);
	FMemory::Memcpy(OutData.DataContainer.ParticleIndices, ParticleIndices, OutData.DataContainer.ParticleIndicesNumShorts * sizeof(uint16));

	// All particle emitter types derived from sprite emitters, so we can fill that data in here too!
	{
		FDynamicSpriteEmitterReplayDataBase* NewReplayData =
			static_cast< FDynamicSpriteEmitterReplayDataBase* >( &OutData );

		NewReplayData->RequiredModule = LODLevel->RequiredModule->CreateRendererResource();
		NewReplayData->MaterialInterface = NULL;	// Must be set by derived implementation
		NewReplayData->InvDeltaSeconds = (LastDeltaTime > UE_KINDA_SMALL_NUMBER) ? (1.0f / LastDeltaTime) : 0.0f;
		NewReplayData->LWCTile = LODLevel->RequiredModule->bUseLocalSpace ? FVector3f::Zero() : Component.GetLWCTile();

		NewReplayData->MaxDrawCount =
			(LODLevel->RequiredModule->bUseMaxDrawCount == true) ? LODLevel->RequiredModule->MaxDrawCount : -1;
		NewReplayData->ScreenAlignment	= LODLevel->RequiredModule->ScreenAlignment;
		NewReplayData->bUseLocalSpace = LODLevel->RequiredModule->bUseLocalSpace;
		NewReplayData->EmitterRenderMode = SpriteTemplate->EmitterRenderMode;
		NewReplayData->DynamicParameterDataOffset = DynamicParameterDataOffset;
		NewReplayData->LightDataOffset = LightDataOffset;
		NewReplayData->LightVolumetricScatteringIntensity = LightVolumetricScatteringIntensity;
		NewReplayData->CameraPayloadOffset = CameraPayloadOffset;

		NewReplayData->SubUVDataOffset = SubUVDataOffset;
		NewReplayData->SubImages_Horizontal = LODLevel->RequiredModule->SubImages_Horizontal;
		NewReplayData->SubImages_Vertical = LODLevel->RequiredModule->SubImages_Vertical;

		NewReplayData->MacroUVOverride.bOverride = LODLevel->RequiredModule->bOverrideSystemMacroUV;
		NewReplayData->MacroUVOverride.Radius = LODLevel->RequiredModule->MacroUVRadius;
		NewReplayData->MacroUVOverride.Position = FVector3f(LODLevel->RequiredModule->MacroUVPosition);
        
		NewReplayData->bLockAxis = false;
		if (bAxisLockEnabled == true)
		{
			NewReplayData->LockAxisFlag = LockAxisFlags;
			if (LockAxisFlags != EPAL_NONE)
			{
				NewReplayData->bLockAxis = true;
			}
		}

		// If there are orbit modules, add the orbit module data
		if (LODLevel->OrbitModules.Num() > 0)
		{
			UParticleLODLevel* HighestLODLevel = SpriteTemplate->LODLevels[0];
			UParticleModuleOrbit* LastOrbit = HighestLODLevel->OrbitModules[LODLevel->OrbitModules.Num() - 1];
			check(LastOrbit);

			uint32* LastOrbitOffset = SpriteTemplate->ModuleOffsetMap.Find(LastOrbit);
			NewReplayData->OrbitModuleOffset = *LastOrbitOffset;
		}

		NewReplayData->EmitterNormalsMode = LODLevel->RequiredModule->EmitterNormalsMode;
		NewReplayData->NormalsSphereCenter = (FVector3f)LODLevel->RequiredModule->NormalsSphereCenter;
		NewReplayData->NormalsCylinderDirection = (FVector3f)LODLevel->RequiredModule->NormalsCylinderDirection;

		NewReplayData->PivotOffset = FVector2f(PivotOffset);

		NewReplayData->bUseVelocityForMotionBlur = LODLevel->RequiredModule->ShouldUseVelocityForMotionBlur();
		NewReplayData->bRemoveHMDRoll = LODLevel->RequiredModule->bRemoveHMDRoll;
		NewReplayData->MinFacingCameraBlendDistance = LODLevel->RequiredModule->MinFacingCameraBlendDistance;
		NewReplayData->MaxFacingCameraBlendDistance = LODLevel->RequiredModule->MaxFacingCameraBlendDistance;
	}


	return true;
}

// Deprecated 5.7
void FParticleEmitterInstance::GatherMaterialRelevance(FMaterialRelevance* OutMaterialRelevance, const UParticleLODLevel* LODLevel, ERHIFeatureLevel::Type InFeatureLevel) const
{
	return GatherMaterialRelevance(OutMaterialRelevance, LODLevel, GetFeatureLevelShaderPlatform_Checked(InFeatureLevel));
}

/**
 * Gathers material relevance flags for this emitter instance.
 * @param OutMaterialRelevance - Pointer to where material relevance flags will be stored.
 * @param LODLevel - The LOD level for which to compute material relevance flags.
 * @param InFeatureLevel - The relevant shader feature level.
 */
void FParticleEmitterInstance::GatherMaterialRelevance(FMaterialRelevance* OutMaterialRelevance, const UParticleLODLevel* LODLevel, EShaderPlatform InShaderPlatform) const
{
	// These will catch the sprite cases...
	if (CurrentMaterial)
	{
		(*OutMaterialRelevance) |= CurrentMaterial->GetRelevance_Concurrent(InShaderPlatform);
	}
	else if (LODLevel->RequiredModule->Material)
	{
		(*OutMaterialRelevance) |= LODLevel->RequiredModule->Material->GetRelevance_Concurrent(InShaderPlatform);
	}
	else
	{
		check( UMaterial::GetDefaultMaterial(MD_Surface) );
		(*OutMaterialRelevance) |= UMaterial::GetDefaultMaterial(MD_Surface)->GetRelevance_Concurrent(InShaderPlatform);
	}
}


// Called on world origin changes
void FParticleEmitterInstance::ApplyWorldOffset(FVector InOffset, bool bWorldShift)
{
	UpdateTransforms();
	
	Location+= InOffset;
	OldLocation+= InOffset;

	UParticleLODLevel* LODLevel = GetCurrentLODLevelChecked();
	if (!LODLevel->RequiredModule->bUseLocalSpace)
	{
		PositionOffsetThisTick = InOffset;
	}
}

void FParticleEmitterInstance::Tick_MaterialOverrides(int32 EmitterIndex)
{
	UParticleLODLevel* LODLevel = SpriteTemplate->GetCurrentLODLevel(this);
	TArrayView<TObjectPtr<UMaterialInterface>> EmitterMaterials = Component.GetEmitterMaterials();

	bool bOverridden = false;
	if( LODLevel && LODLevel->RequiredModule && Component.GetTemplate() )
	{
	        TArray<FName>& NamedOverrides = LODLevel->RequiredModule->NamedMaterialOverrides;
	        const TArray<FNamedEmitterMaterial>& Slots = Component.GetTemplate()->NamedMaterialSlots;
	        if (NamedOverrides.Num() > 0)
	        {
		        //If we have named material overrides then get it's index into the emitter materials array.
		        //Only check for [0] in in the named overrides as most emitter types only need one material. Mesh emitters might use more but they override this function.		
		        for (int32 CheckIdx = 0; CheckIdx < Slots.Num(); ++CheckIdx)
		        {
			        if (NamedOverrides[0] == Slots[CheckIdx].Name)
			        {
				        //Default to the default material for that slot.
				        CurrentMaterial = Slots[CheckIdx].Material;
				        if (EmitterMaterials.IsValidIndex(CheckIdx) && nullptr != EmitterMaterials[CheckIdx])
				        {
					        //This material has been overridden externally, e.g. from a BP so use that one.
					        CurrentMaterial = EmitterMaterials[CheckIdx];
				        }
        
				        bOverridden = true;
				        break;
			        }
		        }
	        }
	}

	if (bOverridden == false)
	{
		if (EmitterMaterials.IsValidIndex(EmitterIndex))
		{
			if (EmitterMaterials[EmitterIndex])
			{
				CurrentMaterial = EmitterMaterials[EmitterIndex];
			}
		}
	}
}

bool FParticleEmitterInstance::UseLocalSpace()
{
	const UParticleLODLevel* LODLevel = GetCurrentLODLevelChecked();
	return LODLevel->RequiredModule->bUseLocalSpace;
}

void FParticleEmitterInstance::GetScreenAlignmentAndScale(int32& OutScreenAlign, FVector& OutScale)
{
	const UParticleLODLevel* LODLevel = GetCurrentLODLevelChecked();
	OutScreenAlign = (int32)LODLevel->RequiredModule->ScreenAlignment;

	OutScale = FVector(1.0f, 1.0f, 1.0f);
	OutScale = Component.GetComponentTransform().GetScale3D();
}

/*-----------------------------------------------------------------------------
	ParticleSpriteEmitterInstance
-----------------------------------------------------------------------------*/
/**
 *	ParticleSpriteEmitterInstance
 *	The structure for a standard sprite emitter instance.
 */
/** Constructor	*/
FParticleSpriteEmitterInstance::FParticleSpriteEmitterInstance(IParticleEmitterInstanceOwner& InComponent) :
	FParticleEmitterInstance(InComponent)
{
}

/** Destructor	*/
FParticleSpriteEmitterInstance::~FParticleSpriteEmitterInstance()
{
}



/**
 *	Retrieves the dynamic data for the emitter
 *	
 *	@param	bSelected					Whether the emitter is selected in the editor
 *  @param InFeatureLevel				The relevant shader feature level.
 *
 *	@return	FDynamicEmitterDataBase*	The dynamic data, or NULL if it shouldn't be rendered
 */
FDynamicEmitterDataBase* FParticleSpriteEmitterInstance::GetDynamicData(bool bSelected, ERHIFeatureLevel::Type InFeatureLevel)
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_ParticleSpriteEmitterInstance_GetDynamicData);

	// It is valid for the LOD level to be NULL here!
	UParticleLODLevel* LODLevel = SpriteTemplate->GetCurrentLODLevel(this);
	if (IsDynamicDataRequired(LODLevel) == false || !bEnabled)
	{
		return NULL;
	}

	// Allocate the dynamic data
	FDynamicSpriteEmitterData* NewEmitterData = new FDynamicSpriteEmitterData(LODLevel->RequiredModule);
	{
		INC_DWORD_STAT(STAT_DynamicEmitterCount);
		INC_DWORD_STAT(STAT_DynamicSpriteCount);
		INC_DWORD_STAT_BY(STAT_DynamicEmitterMem, sizeof(FDynamicSpriteEmitterData));
	}

	// Now fill in the source data
	if( !FillReplayData( NewEmitterData->Source ) )
	{
		delete NewEmitterData;
		return NULL;
	}

	// Setup dynamic render data.  Only call this AFTER filling in source data for the emitter.
	NewEmitterData->Init( bSelected );

	return NewEmitterData;
}

/**
 *	Retrieves replay data for the emitter
 *
 *	@return	The replay data, or NULL on failure
 */
FDynamicEmitterReplayDataBase* FParticleSpriteEmitterInstance::GetReplayData()
{
	if (ActiveParticles <= 0 || !bEnabled)
	{
		return NULL;
	}

	FDynamicEmitterReplayDataBase* NewEmitterReplayData = new FDynamicSpriteEmitterReplayData();
	check( NewEmitterReplayData != NULL );

	if( !FillReplayData( *NewEmitterReplayData ) )
	{
		delete NewEmitterReplayData;
		return NULL;
	}

	return NewEmitterReplayData;
}

/**
 *	Retrieve the allocated size of this instance.
 *
 *	@param	OutNum			The size of this instance
 *	@param	OutMax			The maximum size of this instance
 */
void FParticleSpriteEmitterInstance::GetAllocatedSize(int32& OutNum, int32& OutMax)
{
	int32 Size = sizeof(FParticleSpriteEmitterInstance);
	int32 ActiveParticleDataSize = (ParticleData != NULL) ? (ActiveParticles * ParticleStride) : 0;
	int32 MaxActiveParticleDataSize = (ParticleData != NULL) ? (MaxActiveParticles * ParticleStride) : 0;
	int32 ActiveParticleIndexSize = (ParticleIndices != NULL) ? (ActiveParticles * sizeof(uint16)) : 0;
	int32 MaxActiveParticleIndexSize = (ParticleIndices != NULL) ? (MaxActiveParticles * sizeof(uint16)) : 0;

	OutNum = ActiveParticleDataSize + ActiveParticleIndexSize + Size;
	OutMax = MaxActiveParticleDataSize + MaxActiveParticleIndexSize + Size;
}

/**
 * Returns the size of the object/ resource for display to artists/ LDs in the Editor.
 *
 * @param	Mode	Specifies which resource size should be displayed. ( see EResourceSizeMode::Type )
 * @return	Size of resource as to be displayed to artists/ LDs in the Editor.
 */
void FParticleSpriteEmitterInstance::GetResourceSizeEx(FResourceSizeEx& CumulativeResourceSize)
{
	if (CumulativeResourceSize.GetResourceSizeMode() == EResourceSizeMode::EstimatedTotal || (Component.GetSceneProxy()))
	{
		int32 MaxActiveParticleDataSize = (ParticleData != NULL) ? (MaxActiveParticles * ParticleStride) : 0;
		int32 MaxActiveParticleIndexSize = (ParticleIndices != NULL) ? (MaxActiveParticles * sizeof(uint16)) : 0;
		// Take dynamic data into account as well
		CumulativeResourceSize.AddUnknownMemoryBytes(sizeof(FDynamicSpriteEmitterData));
		CumulativeResourceSize.AddUnknownMemoryBytes(MaxActiveParticleDataSize);								// Copy of the particle data on the render thread
		CumulativeResourceSize.AddUnknownMemoryBytes(MaxActiveParticleIndexSize);								// Copy of the particle indices on the render thread
		CumulativeResourceSize.AddUnknownMemoryBytes(MaxActiveParticles * sizeof(FParticleSpriteVertex));		// The vertex data array

		// Account for dynamic parameter data
		if (DynamicParameterDataOffset > 0)
		{
			CumulativeResourceSize.AddUnknownMemoryBytes(MaxActiveParticles * sizeof(FParticleVertexDynamicParameter));
		}
	}
}

/**
 * Captures dynamic replay data for this particle system.
 *
 * @param	OutData		[Out] Data will be copied here
 *
 * @return Returns true if successful
 */
bool FParticleSpriteEmitterInstance::FillReplayData( FDynamicEmitterReplayDataBase& OutData )
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_ParticleSpriteEmitterInstance_FillReplayData);

	if (ActiveParticles <= 0)
	{
		return false;
	}

	// Call parent implementation first to fill in common particle source data
	if( !FParticleEmitterInstance::FillReplayData( OutData ) )
	{
		return false;
	}

	OutData.eEmitterType = DET_Sprite;

	FDynamicSpriteEmitterReplayData* NewReplayData =
		static_cast< FDynamicSpriteEmitterReplayData* >( &OutData );

	// Get the material instance. If there is none, or the material isn't flagged for use with particle systems, use the DefaultMaterial.
	NewReplayData->MaterialInterface = GetCurrentMaterial();

	return true;
}

UMaterialInterface* FParticleEmitterInstance::GetCurrentMaterial()
{
	UMaterialInterface* RenderMaterial = CurrentMaterial;
	if ((RenderMaterial == NULL) || (RenderMaterial->CheckMaterialUsage_Concurrent(MATUSAGE_ParticleSprites) == false))
	{
		RenderMaterial = UMaterial::GetDefaultMaterial(MD_Surface);
	}
	CurrentMaterial = RenderMaterial;
	return RenderMaterial;
}

/*-----------------------------------------------------------------------------
	ParticleMeshEmitterInstance
-----------------------------------------------------------------------------*/
/**
 *	Structure for mesh emitter instances
 */

/** Constructor	*/
FParticleMeshEmitterInstance::FParticleMeshEmitterInstance(IParticleEmitterInstanceOwner& InComponent) :
	  FParticleEmitterInstance(InComponent)
	, MeshTypeData(NULL)
	, MeshRotationActive(false)
	, MeshRotationOffset(0)
	, MeshMotionBlurOffset(0)
{
}


void FParticleMeshEmitterInstance::InitParameters(UParticleEmitter* InTemplate)
{
	SCOPE_CYCLE_COUNTER(STAT_MeshEmitterInstance_InitParameters);

	FParticleEmitterInstance::InitParameters(InTemplate);

	// Get the type data module
	UParticleLODLevel* LODLevel	= InTemplate->GetLODLevel(0);
	check(LODLevel);
	MeshTypeData = CastChecked<UParticleModuleTypeDataMesh>(LODLevel->TypeDataModule);
	check(MeshTypeData);

	// Grab cached mesh rotation flag from ParticleEmitter template
	MeshRotationActive = InTemplate->bMeshRotationActive;
}

/**
 *	Initialize the instance
 */
void FParticleMeshEmitterInstance::Init()
{
	SCOPE_CYCLE_COUNTER(STAT_MeshEmitterInstance_Init);

	FParticleEmitterInstance::Init();
}

/**
 *	Resize the particle data array
 *
 *	@param	NewMaxActiveParticles	The new size to use
 *
 *	@return	bool					true if the resize was successful
 */
bool FParticleMeshEmitterInstance::Resize(int32 NewMaxActiveParticles, bool bSetMaxActiveCount)
{
	int32 OldMaxActiveParticles = MaxActiveParticles;
	if (FParticleEmitterInstance::Resize(NewMaxActiveParticles, bSetMaxActiveCount) == true)
	{
		if (MeshRotationActive)
		{
			for (int32 i = OldMaxActiveParticles; i < NewMaxActiveParticles; i++)
			{
				DECLARE_PARTICLE(Particle, ParticleData + ParticleStride * ParticleIndices[i]);
				FMeshRotationPayloadData* PayloadData	= (FMeshRotationPayloadData*)((uint8*)&Particle + MeshRotationOffset);
				PayloadData->RotationRateBase			= FVector3f::ZeroVector;
			}
		}
		
		return true;
	}

	return false;
}

/**
 *	Tick the instance.
 *
 *	@param	DeltaTime			The time slice to use
 *	@param	bSuppressSpawning	If true, do not spawn during Tick
 */
void FParticleMeshEmitterInstance::Tick(float DeltaTime, bool bSuppressSpawning)
{
	SCOPE_CYCLE_COUNTER(STAT_MeshTickTime);

	if (bEnabled && MeshMotionBlurOffset)
	{
		for (int32 i = 0; i < ActiveParticles; i++)
		{
			DECLARE_PARTICLE(Particle, ParticleData + ParticleStride * ParticleIndices[i]);

			FMeshRotationPayloadData* RotationPayloadData = (FMeshRotationPayloadData*)((uint8*)&Particle + MeshRotationOffset);
			FMeshMotionBlurPayloadData* MotionBlurPayloadData = (FMeshMotionBlurPayloadData*)((uint8*)&Particle + MeshMotionBlurOffset);

			MotionBlurPayloadData->BaseParticlePrevRotation = Particle.Rotation;
			MotionBlurPayloadData->BaseParticlePrevVelocity = Particle.Velocity;
			MotionBlurPayloadData->BaseParticlePrevSize = Particle.Size;
			MotionBlurPayloadData->PayloadPrevRotation = RotationPayloadData->Rotation;

			if (CameraPayloadOffset)
			{
				const FCameraOffsetParticlePayload* CameraPayload = (const FCameraOffsetParticlePayload*)((const uint8*)&Particle + CameraPayloadOffset);
				MotionBlurPayloadData->PayloadPrevCameraOffset = CameraPayload->Offset;
			}
			else
			{
				MotionBlurPayloadData->PayloadPrevCameraOffset = 0.0f;
			}

			if (OrbitModuleOffset)
			{
				const FOrbitChainModuleInstancePayload* OrbitPayload = (const FOrbitChainModuleInstancePayload*)((const uint8*)&Particle + OrbitModuleOffset);
				MotionBlurPayloadData->PayloadPrevOrbitOffset = OrbitPayload->Offset;
			}
			else
			{
				MotionBlurPayloadData->PayloadPrevOrbitOffset = FVector3f::ZeroVector;
			}
		}
	}

	UParticleLODLevel* LODLevel = GetCurrentLODLevelChecked();
	// See if we are handling mesh rotation
	if (MeshRotationActive && bEnabled)
	{
		// Update the rotation for each particle
		for (int32 i = 0; i < ActiveParticles; i++)
		{
			DECLARE_PARTICLE(Particle, ParticleData + ParticleStride * ParticleIndices[i]);
			FMeshRotationPayloadData* PayloadData	= (FMeshRotationPayloadData*)((uint8*)&Particle + MeshRotationOffset);
			PayloadData->RotationRate				= PayloadData->RotationRateBase;
			if (LODLevel->RequiredModule->ScreenAlignment == PSA_Velocity
				|| LODLevel->RequiredModule->ScreenAlignment == PSA_AwayFromCenter)
			{
				// Determine the rotation to the velocity vector and apply it to the mesh
				FVector	NewDirection	= (FVector)Particle.Velocity;
				
				if (LODLevel->RequiredModule->ScreenAlignment == PSA_Velocity)
				{
					//check if an orbit module should affect the velocity...		
					if (LODLevel->RequiredModule->bOrbitModuleAffectsVelocityAlignment &&
						LODLevel->OrbitModules.Num() > 0)
					{
						UParticleModuleOrbit* LastOrbit = SpriteTemplate->LODLevels[0]->OrbitModules[LODLevel->OrbitModules.Num() - 1];
						check(LastOrbit);
					
						uint32 SpriteOrbitModuleOffset = *SpriteTemplate->ModuleOffsetMap.Find(LastOrbit);
						if (SpriteOrbitModuleOffset != 0)
						{
							const FOrbitChainModuleInstancePayload &OrbitPayload = *(FOrbitChainModuleInstancePayload*)((uint8*)&Particle + SpriteOrbitModuleOffset);

							//this should be our current position
							const FVector NewPos = Particle.Location + (FVector)OrbitPayload.Offset;
							//this should be our previous position
							const FVector OldPos = Particle.OldLocation + (FVector)OrbitPayload.PreviousOffset;

							NewDirection = NewPos - OldPos;
						}
					}
				}
				else if (LODLevel->RequiredModule->ScreenAlignment == PSA_AwayFromCenter)
				{
					NewDirection = Particle.Location;
				}

				NewDirection.Normalize();
				FVector	OldDirection(1.0f, 0.0f, 0.0f);

				FQuat Rotation = FQuat::FindBetweenNormals(OldDirection, NewDirection);
				FVector3f Euler(Rotation.Euler());
				PayloadData->Rotation = PayloadData->InitRotation + Euler;
				PayloadData->Rotation += PayloadData->CurContinuousRotation;
			}
			else // not PSA_Velocity or PSA_AwayfromCenter, so rotation is not reset every tick
			{
				if ((Particle.Flags & STATE_Particle_FreezeRotation) == 0)
				{
					PayloadData->Rotation = PayloadData->InitRotation + PayloadData->CurContinuousRotation;
				}
			}
		}
	}


	// Call the standard tick
	FParticleEmitterInstance::Tick(DeltaTime, bSuppressSpawning);

	if (MeshRotationActive && bEnabled)
	{
		//Must do this (at least) after module update other wise the reset value of RotationRate is used.
		//Probably the other stuff before the module tick should be brought down here too and just leave the RotationRate reset before.
		//Though for the sake of not breaking existing behavior, leave things as they are for now.
		for (int32 i = 0; i < ActiveParticles; i++)
		{
			DECLARE_PARTICLE(Particle, ParticleData + ParticleStride * ParticleIndices[i]);
			FMeshRotationPayloadData* PayloadData = (FMeshRotationPayloadData*)((uint8*)&Particle + MeshRotationOffset);
			PayloadData->CurContinuousRotation += DeltaTime * PayloadData->RotationRate;
		}
	}

	// Remove from the Sprite count... happens because we use the Super::Tick
	DEC_DWORD_STAT_BY(STAT_SpriteParticles, ActiveParticles);
	INC_DWORD_STAT_BY(STAT_MeshParticles, ActiveParticles);
}

void FParticleMeshEmitterInstance::Tick_MaterialOverrides(int32 EmitterIndex)
{
	// we need to do this here, since CurrentMaterials is unique to mesh emitters, so this can't be done from the component. CurrentMaterials 
	// may end up in MeshMaterials, so if this doesn't get updated, rendering may access a garbage collected material

	TArrayView<TObjectPtr<UMaterialInterface>> EmitterMaterials = Component.GetEmitterMaterials();

	// make sure currentmaterials are all set to the emitter material, so we don't end up with GC'd material pointers
	// in MeshMaterials when GetMeshMaterials pushes CurrentMaterials to MeshMaterials
	if (EmitterMaterials.IsValidIndex(EmitterIndex))
	{
		if (EmitterMaterials[EmitterIndex])
		{
			for (UMaterialInterface *& CurMat : CurrentMaterials)
			{
				CurMat = EmitterMaterials[EmitterIndex];
			}
		}
	}	

	UParticleLODLevel* LODLevel = SpriteTemplate->GetCurrentLODLevel(this);
	bool bOverridden = false;
	if (LODLevel && LODLevel->RequiredModule && Component.GetTemplate())
	{
		TArray<FName>& NamedOverrides = LODLevel->RequiredModule->NamedMaterialOverrides;
		const TArray<FNamedEmitterMaterial>& Slots = Component.GetTemplate()->NamedMaterialSlots;
		if (NamedOverrides.Num() > 0)
		{
			CurrentMaterials.SetNumZeroed(NamedOverrides.Num());
			for (int32 MaterialIdx = 0; MaterialIdx < NamedOverrides.Num(); ++MaterialIdx)
			{
				//If we have named material overrides then get it's index into the emitter materials array.	
				for (int32 CheckIdx = 0; CheckIdx < Slots.Num(); ++CheckIdx)
				{
					if (NamedOverrides[MaterialIdx] == Slots[CheckIdx].Name)
					{
						//Default to the default material for that slot.
						CurrentMaterials[MaterialIdx] = Slots[CheckIdx].Material;
						if (EmitterMaterials.IsValidIndex(CheckIdx) && nullptr != EmitterMaterials[CheckIdx])
						{
							//This material has been overridden externally, e.g. from a BP so use that one.
							CurrentMaterials[MaterialIdx] = EmitterMaterials[CheckIdx];
						}

						bOverridden = true;
						break;
					}
				}
			}
		}
	}
}

/**
 *	Update the bounding box for the emitter
 *
 *	@param	DeltaTime		The time slice to use
 */
void FParticleMeshEmitterInstance::UpdateBoundingBox(float DeltaTime)
{
	SCOPE_CYCLE_COUNTER(STAT_ParticleUpdateBounds);
	//@todo. Implement proper bound determination for mesh emitters.
	// Currently, just 'forcing' the mesh size to be taken into account.
	if (ActiveParticles > 0)
	{
		const UParticleSystem* Template = Component.GetTemplate();
		bool bUpdateBox = ((Component.GetIsWarmingUp() == false) && (Template != NULL) && (Template->bUseFixedRelativeBoundingBox == false));

		// Take scale into account
		FVector Scale = Component.GetComponentTransform().GetScale3D();

		// Get the static mesh bounds
		FBoxSphereBounds MeshBound;
		if (Component.GetIsWarmingUp() == false)
		{	
			if (MeshTypeData->Mesh)
			{
				MeshBound = MeshTypeData->Mesh->GetBounds();
			}
			else
			{
				//UE_LOGF(LogParticles, Log, "MeshEmitter with no mesh set?? - %ls", Component.GetTemplate() ? *(Component.GetTemplate()->GetPathName()) : TEXT("??????"));
				MeshBound = FBoxSphereBounds(FVector(0, 0, 0), FVector(0, 0, 0), 0);
			}
		}
		else
		{
			// This isn't used anywhere if the bWarmingUp flag is false, but GCC doesn't like it not touched.
			FMemory::Memzero(&MeshBound, sizeof(FBoxSphereBounds));
		}

		UParticleLODLevel* LODLevel = GetCurrentLODLevelChecked();

		const bool bUseLocalSpace = LODLevel->RequiredModule->bUseLocalSpace;

		const FMatrix ComponentToWorld = bUseLocalSpace 
			? Component.GetComponentToWorld().ToMatrixWithScale() 
			: FMatrix::Identity;

		FVector	NewLocation;
		float	NewRotation;
		if (bUpdateBox)
		{
			ParticleBoundingBox.Init();
		}

		// For each particle, offset the box appropriately 
		FVector MinVal(HALF_WORLD_MAX);
		FVector MaxVal(-HALF_WORLD_MAX);
		
		FPlatformMisc::Prefetch(ParticleData, ParticleStride * ParticleIndices[0]);
		FPlatformMisc::Prefetch(ParticleData, (ParticleIndices[0] * ParticleStride) + PLATFORM_CACHE_LINE_SIZE);

		bool bSkipDoubleSpawnUpdate = !SpriteTemplate->bUseLegacySpawningBehavior;
		for (int32 i=0; i<ActiveParticles; i++)
		{
			DECLARE_PARTICLE(Particle, ParticleData + ParticleStride * ParticleIndices[i]);
			FPlatformMisc::Prefetch(ParticleData, ParticleStride * ParticleIndices[i+1]);
			FPlatformMisc::Prefetch(ParticleData, (ParticleIndices[i+1] * ParticleStride) + PLATFORM_CACHE_LINE_SIZE);

			// Do linear integrator and update bounding box
			Particle.OldLocation = Particle.Location;

			bool bJustSpawned = (Particle.Flags & STATE_Particle_JustSpawned) != 0;
			Particle.Flags &= ~STATE_Particle_JustSpawned;

			//Don't update position for newly spawned particles. They already have a partial update applied during spawn.
			bool bSkipUpdate = bJustSpawned && bSkipDoubleSpawnUpdate;

			if ((Particle.Flags & STATE_Particle_Freeze) == 0 && !bSkipUpdate)
			{
				if ((Particle.Flags & STATE_Particle_FreezeTranslation) == 0)
				{
					NewLocation	= Particle.Location + DeltaTime * (FVector)Particle.Velocity;
				}
				else
				{
					NewLocation = Particle.Location;
				}
				if ((Particle.Flags & STATE_Particle_FreezeRotation) == 0)
				{
					NewRotation	= Particle.Rotation + DeltaTime * Particle.RotationRate;
				}
				else
				{
					NewRotation = Particle.Rotation;
				}
			}
			else
			{
				// Don't move it...
				NewLocation = Particle.Location;
				NewRotation = Particle.Rotation;
			}

			FVector LocalExtent = MeshBound.GetBox().GetExtent() * (FVector)Particle.Size * Scale;

			NewLocation			+= PositionOffsetThisTick;
			Particle.OldLocation+= PositionOffsetThisTick;

			// Do angular integrator, and wrap result to within +/- 2 PI
			Particle.Rotation = FMath::Fmod(NewRotation, 2.f*(float)UE_PI);
			Particle.Location = NewLocation;

			if (bUpdateBox)
			{	
				FVector PositionForBounds = NewLocation;

				if (bUseLocalSpace)
				{
					// Note: building the bounding box in world space as that gives tighter bounds than transforming a local space AABB into world space
					PositionForBounds = ComponentToWorld.TransformPosition(NewLocation);
				}

				MinVal[0] = FMath::Min(MinVal[0], PositionForBounds.X - LocalExtent.X);
				MaxVal[0] = FMath::Max(MaxVal[0], PositionForBounds.X + LocalExtent.X);
				MinVal[1] = FMath::Min(MinVal[1], PositionForBounds.Y - LocalExtent.Y);
				MaxVal[1] = FMath::Max(MaxVal[1], PositionForBounds.Y + LocalExtent.Y);
				MinVal[2] = FMath::Min(MinVal[2], PositionForBounds.Z - LocalExtent.Z);
				MaxVal[2] = FMath::Max(MaxVal[2], PositionForBounds.Z + LocalExtent.Z);
			}
		}

		if (bUpdateBox)
		{	
			ParticleBoundingBox = FBox(MinVal, MaxVal);
		}
	}
}

/**
 *	Retrieved the per-particle bytes that this emitter type requires.
 *
 *	@return	uint32	The number of required bytes for particles in the instance
 */
uint32 FParticleMeshEmitterInstance::RequiredBytes()
{
	uint32 uiBytes = FParticleEmitterInstance::RequiredBytes();

	MeshRotationOffset	= PayloadOffset + uiBytes;
	uiBytes += sizeof(FMeshRotationPayloadData);

	if (MeshTypeData)
	{
		const auto* MeshTD = static_cast<const UParticleModuleTypeDataMesh*>(MeshTypeData);
		if (MeshTypeData->IsMotionBlurEnabled())
		{
			MeshMotionBlurOffset = PayloadOffset + uiBytes;
			uiBytes += sizeof(FMeshMotionBlurPayloadData);
		}
	}

	return uiBytes;
}

/**
 *	Handle any post-spawning actions required by the instance
 *
 *	@param	Particle					The particle that was spawned
 *	@param	InterpolationPercentage		The percentage of the time slice it was spawned at
 *	@param	SpawnTIme					The time it was spawned at
 */
void FParticleMeshEmitterInstance::PostSpawn(FBaseParticle* Particle, float InterpolationPercentage, float SpawnTime)
{
	FParticleEmitterInstance::PostSpawn(Particle, InterpolationPercentage, SpawnTime);
	UParticleLODLevel* LODLevel = GetCurrentLODLevelChecked();

	FMeshRotationPayloadData* PayloadData = (FMeshRotationPayloadData*)((uint8*)Particle + MeshRotationOffset);

	if (LODLevel->RequiredModule->ScreenAlignment == PSA_Velocity
		|| LODLevel->RequiredModule->ScreenAlignment == PSA_AwayFromCenter)
	{
		// Determine the rotation to the velocity vector and apply it to the mesh
		FVector	NewDirection(Particle->Velocity);
		if (LODLevel->RequiredModule->ScreenAlignment == PSA_AwayFromCenter)
		{
			NewDirection = Particle->Location;
		}

		NewDirection.Normalize();
		FVector	OldDirection(1.0f, 0.0f, 0.0f);

		FQuat Rotation	= FQuat::FindBetweenNormals(OldDirection, NewDirection);
		FVector Euler	= Rotation.Euler();

		PayloadData->Rotation.X	+= Euler.X;
		PayloadData->Rotation.Y	+= Euler.Y;
		PayloadData->Rotation.Z	+= Euler.Z;
	}

	FVector InitialOrient = MeshTypeData->RollPitchYawRange.GetValue(SpawnTime, 0, 0, &MeshTypeData->RandomStream);
	PayloadData->InitialOrientation = (FVector3f)InitialOrient;

	if (MeshMotionBlurOffset)
	{
		FMeshRotationPayloadData* RotationPayloadData = (FMeshRotationPayloadData*)((uint8*)Particle + MeshRotationOffset);
		FMeshMotionBlurPayloadData* MotionBlurPayloadData = (FMeshMotionBlurPayloadData*)((uint8*)Particle + MeshMotionBlurOffset);

		MotionBlurPayloadData->BaseParticlePrevRotation = Particle->Rotation;
		MotionBlurPayloadData->BaseParticlePrevVelocity = Particle->Velocity;
		MotionBlurPayloadData->BaseParticlePrevSize = Particle->Size;
		MotionBlurPayloadData->PayloadPrevRotation = RotationPayloadData->Rotation;

		if (CameraPayloadOffset)
		{
			const FCameraOffsetParticlePayload* CameraPayload = (const FCameraOffsetParticlePayload*)((const uint8*)Particle + CameraPayloadOffset);
			MotionBlurPayloadData->PayloadPrevCameraOffset = CameraPayload->Offset;
		}
		else
		{
			MotionBlurPayloadData->PayloadPrevCameraOffset = 0.0f;
		}

		if (OrbitModuleOffset)
		{
			const FOrbitChainModuleInstancePayload* OrbitPayload = (const FOrbitChainModuleInstancePayload*)((const uint8*)Particle + OrbitModuleOffset);
			MotionBlurPayloadData->PayloadPrevOrbitOffset = OrbitPayload->Offset;
		}
		else
		{
			MotionBlurPayloadData->PayloadPrevOrbitOffset = FVector3f::ZeroVector;
		}
	}
}

bool FParticleMeshEmitterInstance::IsDynamicDataRequired(UParticleLODLevel* InCurrentLODLevel)
{
	return MeshTypeData->Mesh != NULL
		&& MeshTypeData->Mesh->HasValidRenderData(false)
		&& FParticleEmitterInstance::IsDynamicDataRequired(InCurrentLODLevel);
}

/**
 *	Retrieves the dynamic data for the emitter
 *	
 *	@param	bSelected					Whether the emitter is selected in the editor
 *  @param InFeatureLevel				The relevant shader feature level.
 *
 *	@return	FDynamicEmitterDataBase*	The dynamic data, or NULL if it shouldn't be rendered
 */
FDynamicEmitterDataBase* FParticleMeshEmitterInstance::GetDynamicData(bool bSelected, ERHIFeatureLevel::Type InFeatureLevel)
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_ParticleMeshEmitterInstance_GetDynamicData);

	// It is safe for LOD level to be NULL here!
	UParticleLODLevel* LODLevel = SpriteTemplate->GetCurrentLODLevel(this);
	if (IsDynamicDataRequired(LODLevel) == false || !bEnabled)
	{
		return NULL;
	}

	// Allocate the dynamic data
	FDynamicMeshEmitterData* NewEmitterData = new FDynamicMeshEmitterData(LODLevel->RequiredModule);
	{
		INC_DWORD_STAT(STAT_DynamicEmitterCount);
		INC_DWORD_STAT(STAT_DynamicMeshCount);
		INC_DWORD_STAT_BY(STAT_DynamicEmitterMem, sizeof(FDynamicMeshEmitterData));
	}

	// Now fill in the source data
	if( !FillReplayData( NewEmitterData->Source ) )
	{
		delete NewEmitterData;
		return NULL;
	}


	// Setup dynamic render data.  Only call this AFTER filling in source data for the emitter.
	NewEmitterData->Init(
		bSelected,
		this,
		MeshTypeData->Mesh,
		MeshTypeData->bUseStaticMeshLODs,
		MeshTypeData->LODSizeScale,
		InFeatureLevel
		);

	return NewEmitterData;
}

/**
 *	Retrieves replay data for the emitter
 *
 *	@return	The replay data, or NULL on failure
 */
FDynamicEmitterReplayDataBase* FParticleMeshEmitterInstance::GetReplayData()
{
	if (ActiveParticles <= 0 || !bEnabled)
	{
		return NULL;
	}

	FDynamicEmitterReplayDataBase* NewEmitterReplayData = new FDynamicMeshEmitterReplayData();
	check( NewEmitterReplayData != NULL );

	if( !FillReplayData( *NewEmitterReplayData ) )
	{
		delete NewEmitterReplayData;
		return NULL;
	}

	return NewEmitterReplayData;
}

/**
 *	Retrieve the allocated size of this instance.
 *
 *	@param	OutNum			The size of this instance
 *	@param	OutMax			The maximum size of this instance
 */
void FParticleMeshEmitterInstance::GetAllocatedSize(int32& OutNum, int32& OutMax)
{
	int32 Size = sizeof(FParticleMeshEmitterInstance);
	int32 ActiveParticleDataSize = (ParticleData != NULL) ? (ActiveParticles * ParticleStride) : 0;
	int32 MaxActiveParticleDataSize = (ParticleData != NULL) ? (MaxActiveParticles * ParticleStride) : 0;
	int32 ActiveParticleIndexSize = (ParticleIndices != NULL) ? (ActiveParticles * sizeof(uint16)) : 0;
	int32 MaxActiveParticleIndexSize = (ParticleIndices != NULL) ? (MaxActiveParticles * sizeof(uint16)) : 0;

	OutNum = ActiveParticleDataSize + ActiveParticleIndexSize + Size;
	OutMax = MaxActiveParticleDataSize + MaxActiveParticleIndexSize + Size;
}

/**
 * Returns the size of the object/ resource for display to artists/ LDs in the Editor.
 *
 * @param	Mode	Specifies which resource size should be displayed. ( see EResourceSizeMode::Type )
 * @return  Size of resource as to be displayed to artists/ LDs in the Editor.
 */
void FParticleMeshEmitterInstance::GetResourceSizeEx(FResourceSizeEx& CumulativeResourceSize)
{
	if (CumulativeResourceSize.GetResourceSizeMode() == EResourceSizeMode::EstimatedTotal || (Component.GetSceneProxy()))
	{
		int32 MaxActiveParticleDataSize = (ParticleData != NULL) ? (MaxActiveParticles * ParticleStride) : 0;
		int32 MaxActiveParticleIndexSize = (ParticleIndices != NULL) ? (MaxActiveParticles * sizeof(uint16)) : 0;
		// Take dynamic data into account as well
		CumulativeResourceSize.AddUnknownMemoryBytes(sizeof(FDynamicMeshEmitterData));
		CumulativeResourceSize.AddUnknownMemoryBytes(MaxActiveParticleDataSize);								// Copy of the particle data on the render thread
		CumulativeResourceSize.AddUnknownMemoryBytes(MaxActiveParticleIndexSize);								// Copy of the particle indices on the render thread
	}
}

/**
 * Sets the materials with which mesh particles should be rendered.
 */
void FParticleMeshEmitterInstance::SetMeshMaterials( const TArray<UMaterialInterface*>& InMaterials )
{
	#if !WITH_STATE_STREAM
	check(IsInGameThread());
	#endif

	CurrentMaterials = InMaterials;
}

// Deprecated 5.7
void FParticleMeshEmitterInstance::GatherMaterialRelevance(FMaterialRelevance* OutMaterialRelevance, const UParticleLODLevel* LODLevel, ERHIFeatureLevel::Type InFeatureLevel) const
{
	return GatherMaterialRelevance(OutMaterialRelevance, LODLevel, GetFeatureLevelShaderPlatform_Checked(InFeatureLevel));
}

/**
 * Gathers material relevance flags for this emitter instance.
 * @param OutMaterialRelevance - Pointer to where material relevance flags will be stored.
 * @param LODLevel - The LOD level for which to compute material relevance flags.
 * @param InFeatureLevel - The relevant shader feature level.
 */
void FParticleMeshEmitterInstance::GatherMaterialRelevance( FMaterialRelevance* OutMaterialRelevance, const UParticleLODLevel* LODLevel, EShaderPlatform InShaderPlatform ) const
{
	TArray<UMaterialInterface*,TInlineAllocator<2> > Materials;
	GetMeshMaterials(Materials, LODLevel, true); // Allow log issues since GatherMaterialRelevance is only called when the proxy is created.
	for (int32 MaterialIndex = 0; MaterialIndex < Materials.Num(); ++MaterialIndex)
	{
		(*OutMaterialRelevance) |= Materials[MaterialIndex]->GetRelevance_Concurrent(InShaderPlatform);
	}
}

// Deprecated in 5.7
void FParticleMeshEmitterInstance::GetMeshMaterials(
	TArray<UMaterialInterface*, TInlineAllocator<2> >& OutMaterials,
	const UParticleLODLevel* LODLevel,
	ERHIFeatureLevel::Type InFeatureLevel,
	bool bLogWarnings
) const
{
	GetMeshMaterials(OutMaterials, LODLevel, bLogWarnings);
}

void FParticleMeshEmitterInstance::GetMeshMaterials(
	TArray<UMaterialInterface*,TInlineAllocator<2> >& OutMaterials,
	const UParticleLODLevel* LODLevel,
	bool bLogWarnings
	) const
{
	if (MeshTypeData && MeshTypeData->Mesh)
	{
		const FStaticMeshLODResources& LODModel = MeshTypeData->Mesh->GetRenderData()->LODResources[0];

		// Gather the materials applied to the LOD.
		for (int32 SectionIndex = 0; SectionIndex < LODModel.Sections.Num(); SectionIndex++)
		{
			UMaterialInterface* Material = NULL;

			if (SectionIndex < CurrentMaterials.Num())
			{
				Material = CurrentMaterials[SectionIndex];
			}

			// See if there is a mesh material module.
			if (Material == NULL)
			{
				for (int32 ModuleIndex = 0; ModuleIndex < LODLevel->Modules.Num(); ModuleIndex++)
				{
					UParticleModuleMeshMaterial* MeshMatModule = Cast<UParticleModuleMeshMaterial>(LODLevel->Modules[ModuleIndex]);
					if (MeshMatModule && MeshMatModule->bEnabled)
					{
						if (SectionIndex < MeshMatModule->MeshMaterials.Num())
						{
							Material = MeshMatModule->MeshMaterials[SectionIndex];
							break;
						}
					}
				}
			}

			// Overriding the material?
			if (Material == NULL && MeshTypeData->bOverrideMaterial == true)
			{
				Material = CurrentMaterial ? CurrentMaterial : ToRawPtr(LODLevel->RequiredModule->Material);
			}

			// Use the material set on the mesh.
			if (Material == NULL)
			{
				Material = MeshTypeData->Mesh->GetMaterial(LODModel.Sections[SectionIndex].MaterialIndex);
			}

			// Use the default material...
			if (Material == NULL)
			{
				Material = UMaterial::GetDefaultMaterial(MD_Surface);
			}

			check( Material );
			OutMaterials.Add(Material);
		}
	}
}

/**
 * Captures dynamic replay data for this particle system.
 *
 * @param	OutData		[Out] Data will be copied here
 *
 * @return Returns true if successful
 */
bool FParticleMeshEmitterInstance::FillReplayData( FDynamicEmitterReplayDataBase& OutData )
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_ParticleMeshEmitterInstance_FillReplayData);

	// Call parent implementation first to fill in common particle source data
	if( !FParticleEmitterInstance::FillReplayData( OutData ) )
	{
		return false;
	}

	// Grab the LOD level
	UParticleLODLevel* LODLevel = SpriteTemplate->GetCurrentLODLevel(this);
	if ((LODLevel == NULL) || (LODLevel->bEnabled == false))
	{
		return false;
	}

	OutData.eEmitterType = DET_Mesh;

	FDynamicMeshEmitterReplayData* NewReplayData =
		static_cast< FDynamicMeshEmitterReplayData* >( &OutData );

	// Mesh settings
	NewReplayData->MaterialInterface = nullptr;
	NewReplayData->bScaleUV = LODLevel->RequiredModule->bScaleUV;
	NewReplayData->SubUVInterpMethod = LODLevel->RequiredModule->InterpolationMethod;
	NewReplayData->SubUVDataOffset = SubUVDataOffset;
	NewReplayData->SubImages_Horizontal = LODLevel->RequiredModule->SubImages_Horizontal;
	NewReplayData->SubImages_Vertical = LODLevel->RequiredModule->SubImages_Vertical;
	NewReplayData->MeshRotationOffset = MeshRotationOffset;
	NewReplayData->MeshMotionBlurOffset = MeshMotionBlurOffset;
	NewReplayData->bMeshRotationActive = MeshRotationActive;
	NewReplayData->MeshAlignment = MeshTypeData->MeshAlignment;

	// Scale needs to be handled in a special way for meshes.  The parent implementation set this
	// itself, but we'll recompute it here.
	NewReplayData->Scale = FVector3f::OneVector;
	{
		check(SpriteTemplate);
		UParticleLODLevel* LODLevel2 = SpriteTemplate->GetCurrentLODLevel(this);
		check(LODLevel2);
		check(LODLevel2->RequiredModule);
		// Take scale into account
		if (LODLevel2->RequiredModule->bUseLocalSpace == false)
		{
			if (!bIgnoreComponentScale)
			{
				NewReplayData->Scale = (FVector3f)Component.GetComponentTransform().GetScale3D();
			}
		}
	}

	// See if the new mesh locked axis is being used...
	if (MeshTypeData->AxisLockOption == EPAL_NONE)
	{
		if (bAxisLockEnabled)
		{
			NewReplayData->LockAxisFlag = LockAxisFlags;
			if (LockAxisFlags != EPAL_NONE)
			{
				NewReplayData->bLockAxis = true;
				switch (LockAxisFlags)
				{
				case EPAL_X:
					NewReplayData->LockedAxis = FVector3f(1,0,0);
					break;
				case EPAL_Y:
					NewReplayData->LockedAxis = FVector3f(0,1,0);
					break;
				case EPAL_NEGATIVE_X:
					NewReplayData->LockedAxis = FVector3f(-1,0,0);
					break;
				case EPAL_NEGATIVE_Y:
					NewReplayData->LockedAxis = FVector3f(0,-1,0);
					break;
				case EPAL_NEGATIVE_Z:
					NewReplayData->LockedAxis = FVector3f(0,0,-1);
					break;
				case EPAL_Z:
				case EPAL_NONE:
				default:
					NewReplayData->LockedAxis = FVector3f(0,0,1);
					break;
				}
			}
		}
	}

	return true;
}


void* FDynamicEmitterDataBase::operator new(size_t AllocSize)
{
	return FastParticleSmallBlockAlloc(AllocSize);
}

void FDynamicEmitterDataBase::operator delete(void *RawMemory, size_t AllocSize)
{
	FastParticleSmallBlockFree(RawMemory, AllocSize);
}	

void* FParticleDynamicData::operator new(size_t AllocSize)
{
	return FMemory::Malloc(AllocSize);
}

void FParticleDynamicData::operator delete(void *RawMemory, size_t AllocSize)
{
	FMemory::Free(RawMemory);
}	

FDynamicEmitterDataBase::FDynamicEmitterDataBase(const UParticleModuleRequired* RequiredModule)
	: bSelected(false)
	, EmitterIndex(INDEX_NONE)
{
}

FDynamicSpriteEmitterReplayDataBase::FDynamicSpriteEmitterReplayDataBase()
	: MaterialInterface(nullptr)
	, RequiredModule(nullptr)
	, NormalsSphereCenter(FVector::ZeroVector)
	, NormalsCylinderDirection(FVector::ZeroVector)
	, InvDeltaSeconds(0.0f)
	, MaxDrawCount(0)
	, OrbitModuleOffset(0)
	, DynamicParameterDataOffset(0)
	, LightDataOffset(0)
	, LightVolumetricScatteringIntensity(0)
	, CameraPayloadOffset(0)
	, SubUVDataOffset(0)
	, SubImages_Horizontal(1)
	, SubImages_Vertical(1)
	, bUseLocalSpace(false)
	, bLockAxis(false)
	, ScreenAlignment(0)
	, LockAxisFlag(0)
	, EmitterRenderMode(0)
	, EmitterNormalsMode(0)
	, PivotOffset(-0.5f, -0.5f)
	, bUseVelocityForMotionBlur(false)
	, bRemoveHMDRoll(false)
	, MinFacingCameraBlendDistance(0.f)
	, MaxFacingCameraBlendDistance(0.f)
{
}


FDynamicSpriteEmitterReplayDataBase::~FDynamicSpriteEmitterReplayDataBase()
{
	delete RequiredModule;
}

/** FDynamicSpriteEmitterReplayDataBase Serialization */
void FDynamicSpriteEmitterReplayDataBase::Serialize( FArchive& Ar )
{
	// Call parent implementation
	FDynamicEmitterReplayDataBase::Serialize( Ar );

	Ar << ScreenAlignment;
	Ar << bUseLocalSpace;
	Ar << bLockAxis;
	Ar << LockAxisFlag;
	Ar << MaxDrawCount;

	int32 EmitterRenderModeInt = EmitterRenderMode;
	Ar << EmitterRenderModeInt;
	EmitterRenderMode = EmitterRenderModeInt;

	Ar << OrbitModuleOffset;
	Ar << DynamicParameterDataOffset;
	Ar << LightDataOffset;
	Ar << LightVolumetricScatteringIntensity;
	Ar << CameraPayloadOffset;

	Ar << EmitterNormalsMode;
	Ar << NormalsSphereCenter;
	Ar << NormalsCylinderDirection;

	Ar << MaterialInterface;

	Ar << PivotOffset;

	Ar << bUseVelocityForMotionBlur;
	Ar << bRemoveHMDRoll;
	Ar << MinFacingCameraBlendDistance;
	Ar << MaxFacingCameraBlendDistance;
}
