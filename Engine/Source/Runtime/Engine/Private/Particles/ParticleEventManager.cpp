// Copyright Epic Games, Inc. All Rights Reserved.

#include "Particles/ParticleEventManager.h"
#include "Engine/World.h"
#include "Particles/Emitter.h"
#include "Particles/ParticleSystemComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ParticleEventManager)

/*-----------------------------------------------------------------------------
	AParticleEventManager implementation.
-----------------------------------------------------------------------------*/
AParticleEventManager::AParticleEventManager(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	SetCanBeDamaged(false);
	bReplayRewindable = true;
}

void AParticleEventManager::EndPlay(const EEndPlayReason::Type EndPlayReason)
{
	Super::EndPlay(EndPlayReason);
	if (EndPlayReason == EEndPlayReason::Destroyed)
	{
		if (UWorld* World = GetWorld())
		{
			if (World->MyParticleEventManager == this)
			{
				World->MyParticleEventManager = nullptr;
			}
		}
	}
}

void AParticleEventManager::HandleParticleSpawnEvents( UParticleSystemComponent* Component, const TArray<FParticleEventSpawnData>& SpawnEvents )
{
	AEmitter* Emitter = Cast<AEmitter>(Component->GetOwner());
	for (int32 EventIndex = 0; EventIndex < SpawnEvents.Num(); ++EventIndex)
	{
		const FParticleEventSpawnData& SpawnEvent = SpawnEvents[EventIndex];
		Component->OnParticleSpawn.Broadcast(SpawnEvent.EventName, SpawnEvent.EmitterTime, SpawnEvent.Location, SpawnEvent.Velocity);
		if (Emitter)
		{
			Emitter->OnParticleSpawn.Broadcast(SpawnEvent.EventName, SpawnEvent.EmitterTime, SpawnEvent.Location, SpawnEvent.Velocity);
		}
	}
}

void AParticleEventManager::HandleParticleDeathEvents( UParticleSystemComponent* Component, const TArray<FParticleEventDeathData>& DeathEvents )
{
	AEmitter* Emitter = Cast<AEmitter>(Component->GetOwner());
	for (int32 EventIndex = 0; EventIndex < DeathEvents.Num(); ++EventIndex)
	{
		const FParticleEventDeathData& DeathEvent = DeathEvents[EventIndex];
		Component->OnParticleDeath.Broadcast(DeathEvent.EventName, DeathEvent.EmitterTime, DeathEvent.ParticleTime, DeathEvent.Location, DeathEvent.Velocity, DeathEvent.Direction);
		if (Emitter)
		{
			Emitter->OnParticleDeath.Broadcast(DeathEvent.EventName, DeathEvent.EmitterTime, DeathEvent.ParticleTime, DeathEvent.Location, DeathEvent.Velocity, DeathEvent.Direction);
		}
	}
}

void AParticleEventManager::HandleParticleCollisionEvents( UParticleSystemComponent* Component, const TArray<FParticleEventCollideData>& CollisionEvents )
{
	AEmitter* Emitter = Cast<AEmitter>(Component->GetOwner());
	for (int32 EventIndex = 0; EventIndex < CollisionEvents.Num(); ++EventIndex)
	{
		const FParticleEventCollideData& CollisionEvent = CollisionEvents[EventIndex];
		Component->OnParticleCollide.Broadcast(CollisionEvent.EventName, CollisionEvent.EmitterTime, CollisionEvent.ParticleTime, CollisionEvent.Location, CollisionEvent.Velocity, CollisionEvent.Direction, CollisionEvent.Normal, CollisionEvent.BoneName, CollisionEvent.PhysMat);
		if (Emitter)
		{
			Emitter->OnParticleCollide.Broadcast(CollisionEvent.EventName, CollisionEvent.EmitterTime, CollisionEvent.ParticleTime, CollisionEvent.Location, CollisionEvent.Velocity, CollisionEvent.Direction, CollisionEvent.Normal, CollisionEvent.BoneName, CollisionEvent.PhysMat);
		}
	}
}

void AParticleEventManager::HandleParticleBurstEvents( UParticleSystemComponent* Component, const TArray<FParticleEventBurstData>& BurstEvents )
{
	AEmitter* Emitter = Cast<AEmitter>(Component->GetOwner());
	for (int32 EventIndex = 0; EventIndex < BurstEvents.Num(); ++EventIndex)
	{
		const FParticleEventBurstData& BurstEvent = BurstEvents[EventIndex];
		Component->OnParticleBurst.Broadcast(BurstEvent.EventName, BurstEvent.EmitterTime, BurstEvent.ParticleCount);
		if (Emitter)
		{
			Emitter->OnParticleBurst.Broadcast(BurstEvent.EventName, BurstEvent.EmitterTime, BurstEvent.ParticleCount);
		}
	}
}

