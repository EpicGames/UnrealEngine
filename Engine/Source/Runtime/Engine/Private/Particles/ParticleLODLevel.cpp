// Copyright Epic Games, Inc. All Rights Reserved.

#include "Particles/ParticleLODLevel.h"
#include "Engine/StaticMesh.h"
#include "ParticleEmitterInstances.h"
#include "Particles/Event/ParticleModuleEventGenerator.h"
#include "Particles/Event/ParticleModuleEventReceiverBase.h"
#include "Particles/Lifetime/ParticleModuleLifetimeBase.h"
#include "Particles/Material/ParticleModuleMeshMaterial.h"
#include "Particles/Orbit/ParticleModuleOrbit.h"
#include "Particles/ParticleModuleRequired.h"
#include "Particles/Spawn/ParticleModuleSpawn.h"
#include "Particles/TypeData/ParticleModuleTypeDataBase.h"
#include "Particles/TypeData/ParticleModuleTypeDataMesh.h"
#include "StaticMeshResources.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ParticleLODLevel)

////////////////////////////////////////////////////////////////////////////////////////////////////

UParticleLODLevel::UParticleLODLevel(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bEnabled = true;
	ConvertedModules = true;
	PeakActiveParticles = 0;
}


void UParticleLODLevel::CompileModules( FParticleEmitterBuildInfo& EmitterBuildInfo )
{
	check( RequiredModule );
	check( SpawnModule );

	// Store a few special modules.
	EmitterBuildInfo.RequiredModule = RequiredModule;
	EmitterBuildInfo.SpawnModule = SpawnModule;

	// Compile those special modules.
	RequiredModule->CompileModule( EmitterBuildInfo );
	if ( SpawnModule->bEnabled )
	{
		SpawnModule->CompileModule( EmitterBuildInfo );
	}

	// Compile all remaining modules.
	const int32 ModuleCount = Modules.Num();
	for ( int32 ModuleIndex = 0; ModuleIndex < ModuleCount; ++ModuleIndex )
	{
		UParticleModule* Module = Modules[ModuleIndex];
		if ( Module && Module->bEnabled )
		{
			Module->CompileModule( EmitterBuildInfo );
		}
	}

	// Estimate the maximum number of active particles.
	EmitterBuildInfo.EstimatedMaxActiveParticleCount = CalculateMaxActiveParticleCount();
}

bool UParticleLODLevel::IsPostLoadThreadSafe() const
{
	return false;
}

void UParticleLODLevel::PostLoad()
{
	Super::PostLoad();

#if WITH_EDITORONLY_DATA
	checkf(SpawnModule, TEXT("Missing spawn module on %s (%s)"), *(GetPathName()), 
		(GetOuter() ? (GetOuter()->GetOuter() ? *(GetOuter()->GetOuter()->GetPathName()) : *(GetOuter()->GetPathName())) : TEXT("???")));
#endif // WITH_EDITORONLY_DATA

	{
		RequiredModule->ConditionalPostLoad();
	}
	if ( SpawnModule )
	{
		SpawnModule->ConditionalPostLoad();
	}

	for (UParticleModule* ParticleModule : Modules)
	{
		ParticleModule->ConditionalPostLoad();
	}

	// shouldn't ever set another UObjects serialized variable in post load
	// this causes determinisitc cooking issues due to load order being different
	/*if (RequiredModule)
	{
		RequiredModule->ConditionalPostLoad();
		if (RequiredModule->bEnabled != bEnabled)
		{
			RequiredModule->bEnabled = bEnabled;
		}
	}*/
}

void UParticleLODLevel::UpdateModuleLists()
{
	LLM_SCOPE(ELLMTag::Particles);

	SpawningModules.Empty();
	SpawnModules.Empty();
	UpdateModules.Empty();
	OrbitModules.Empty();
	EventReceiverModules.Empty();
	EventGenerator = NULL;

	UParticleModule* Module;
	int32 TypeDataModuleIndex = -1;

	for (int32 i = 0; i < Modules.Num(); i++)
	{
		Module = Modules[i];
		if (!Module)
		{
			continue;
		}

		if (Module->bSpawnModule)
		{
			SpawnModules.Add(Module);
		}
		if (Module->bUpdateModule || Module->bFinalUpdateModule)
		{
			UpdateModules.Add(Module);
		}

		if (Module->IsA(UParticleModuleTypeDataBase::StaticClass()))
		{
			TypeDataModule = CastChecked<UParticleModuleTypeDataBase>(Module);
			if (!Module->bSpawnModule && !Module->bUpdateModule)
			{
				// For now, remove it from the list and set it as the TypeDataModule
				TypeDataModuleIndex = i;
			}
		}
		else
		if (Module->IsA(UParticleModuleSpawnBase::StaticClass()))
		{
			UParticleModuleSpawnBase* SpawnBase = CastChecked<UParticleModuleSpawnBase>(Module);
			SpawningModules.Add(SpawnBase);
		}
		else
		if (Module->IsA(UParticleModuleOrbit::StaticClass()))
		{
			UParticleModuleOrbit* Orbit = CastChecked<UParticleModuleOrbit>(Module);
			OrbitModules.Add(Orbit);
		}
		else
		if (Module->IsA(UParticleModuleEventGenerator::StaticClass()))
		{
			EventGenerator = CastChecked<UParticleModuleEventGenerator>(Module);
		}
		else
		if (Module->IsA(UParticleModuleEventReceiverBase::StaticClass()))
		{
			UParticleModuleEventReceiverBase* Event = CastChecked<UParticleModuleEventReceiverBase>(Module);
			EventReceiverModules.Add(Event);
		}
	}

	if (EventGenerator)
	{
		// Force the event generator module to the top of the module stack...
		Modules.RemoveSingle(EventGenerator);
		Modules.Insert(EventGenerator, 0);
	}

	if (TypeDataModuleIndex != -1)
	{
		Modules.RemoveAt(TypeDataModuleIndex);
	}

	if (TypeDataModule /**&& (Level == 0)**/)
	{
		UParticleModuleTypeDataMesh* MeshTD = Cast<UParticleModuleTypeDataMesh>(TypeDataModule);
		if (MeshTD
			&& MeshTD->Mesh
			&& MeshTD->Mesh->HasValidRenderData(false))
		{
			UParticleSpriteEmitter* SpriteEmitter = Cast<UParticleSpriteEmitter>(GetOuter());
			if (SpriteEmitter && (MeshTD->bOverrideMaterial == false))
			{
				FStaticMeshSection& Section = MeshTD->Mesh->GetRenderData()->LODResources[0].Sections[0];
				UMaterialInterface* Material = MeshTD->Mesh->GetMaterial(Section.MaterialIndex);
				if (Material)
				{
					RequiredModule->Material = Material;
				}
			}
		}
	}
}


bool UParticleLODLevel::GenerateFromLODLevel(UParticleLODLevel* SourceLODLevel, float Percentage, bool bGenerateModuleData)
{
	// See if there are already modules in place
	if (Modules.Num() > 0)
	{
		UE_LOGF(LogParticles, Log, "ERROR? - GenerateFromLODLevel - modules already present!");
		return false;
	}

	bool	bResult	= true;

	// Allocate slots in the array...
	Modules.InsertZeroed(0, SourceLODLevel->Modules.Num());

	// Set the enabled flag
	bEnabled = SourceLODLevel->bEnabled;

	// Set up for undo/redo!
	SetFlags(RF_Transactional);

	// Required module...
	RequiredModule = CastChecked<UParticleModuleRequired>(
		SourceLODLevel->RequiredModule->GenerateLODModule(SourceLODLevel, this, Percentage, bGenerateModuleData));

	// Spawn module...
	SpawnModule = CastChecked<UParticleModuleSpawn>(
		SourceLODLevel->SpawnModule->GenerateLODModule(SourceLODLevel, this, Percentage, bGenerateModuleData));

	// TypeData module, if present...
	if (SourceLODLevel->TypeDataModule)
	{
		TypeDataModule = 
			CastChecked<UParticleModuleTypeDataBase>(
			SourceLODLevel->TypeDataModule->GenerateLODModule(SourceLODLevel, this, Percentage, bGenerateModuleData));
		check(TypeDataModule == SourceLODLevel->TypeDataModule); // Code expects typedata to be the same across LODs
	}

	// The remaining modules...
	for (int32 ModuleIndex = 0; ModuleIndex < SourceLODLevel->Modules.Num(); ModuleIndex++)
	{
		if (SourceLODLevel->Modules[ModuleIndex])
		{
			Modules[ModuleIndex] = SourceLODLevel->Modules[ModuleIndex]->GenerateLODModule(SourceLODLevel, this, Percentage, bGenerateModuleData);
		}
		else
		{
			Modules[ModuleIndex] = NULL;
		}
	}

	return bResult;
}


int32	UParticleLODLevel::CalculateMaxActiveParticleCount()
{
	check(RequiredModule != NULL);

	// Determine the lifetime for particles coming from the emitter
	float ParticleLifetime = 0.0f;
	float MaxSpawnRate = SpawnModule->GetEstimatedSpawnRate();
	int32 MaxBurstCount = SpawnModule->GetMaximumBurstCount();
	for (int32 ModuleIndex = 0; ModuleIndex < Modules.Num(); ModuleIndex++)
	{
		UParticleModuleLifetimeBase* LifetimeMod = Cast<UParticleModuleLifetimeBase>(Modules[ModuleIndex]);
		if (LifetimeMod != NULL)
		{
			ParticleLifetime += LifetimeMod->GetMaxLifetime();
		}

		UParticleModuleSpawnBase* SpawnMod = Cast<UParticleModuleSpawnBase>(Modules[ModuleIndex]);
		if (SpawnMod != NULL)
		{
			MaxSpawnRate += SpawnMod->GetEstimatedSpawnRate();
			MaxBurstCount += SpawnMod->GetMaximumBurstCount();
		}
	}

	// Determine the maximum duration for this particle system
	float MaxDuration = 0.0f;
	float TotalDuration = 0.0f;
	int32 TotalLoops = 0;
	if (RequiredModule != NULL)
	{
		// We don't care about delay wrt spawning...
		MaxDuration = FMath::Max<float>(RequiredModule->EmitterDuration, RequiredModule->EmitterDurationLow);
		TotalLoops = RequiredModule->EmitterLoops;
		TotalDuration = MaxDuration * TotalLoops;
	}

	// Determine the max
	int32 MaxAPC = 0;

	if (TotalDuration != 0.0f)
	{
		if (TotalLoops == 1)
		{
			// Special case for one loop... 
			if (ParticleLifetime < MaxDuration)
			{
				MaxAPC += FMath::CeilToInt(ParticleLifetime * MaxSpawnRate);
			}
			else
			{
				MaxAPC += FMath::CeilToInt(MaxDuration * MaxSpawnRate);
			}
			// Safety zone...
			MaxAPC += 1;
			// Add in the bursts...
			MaxAPC += MaxBurstCount;
		}
		else
		{
			if (ParticleLifetime < MaxDuration)
			{
				MaxAPC += FMath::CeilToInt(ParticleLifetime * MaxSpawnRate);
			}
			else
			{
				MaxAPC += (FMath::CeilToInt(FMath::CeilToInt(MaxDuration * MaxSpawnRate) * ParticleLifetime));
			}
			// Safety zone...
			MaxAPC += 1;
			// Add in the bursts...
			MaxAPC += MaxBurstCount;
			if (ParticleLifetime > MaxDuration)
			{
				MaxAPC += MaxBurstCount * FMath::CeilToInt(ParticleLifetime - MaxDuration);
			}
		}
	}
	else
	{
		// We are infinite looping... 
		// Single loop case is all we will worry about. Safer base estimate - but not ideal.
		if (ParticleLifetime < MaxDuration)
		{
			MaxAPC += FMath::CeilToInt(ParticleLifetime * FMath::CeilToInt(MaxSpawnRate));
		}
		else
		{
			if (ParticleLifetime != 0.0f)
			{
				if (ParticleLifetime <= MaxDuration)
				{
					MaxAPC += FMath::CeilToInt(MaxDuration * MaxSpawnRate);
				}
				else //if (ParticleLifetime > MaxDuration)
				{
					MaxAPC += FMath::CeilToInt(MaxDuration * MaxSpawnRate) * ParticleLifetime;
				}
			}
			else
			{
				// No lifetime, no duration...
				MaxAPC += FMath::CeilToInt(MaxSpawnRate);
			}
		}
		// Safety zone...
		MaxAPC += FMath::Max<int32>(FMath::CeilToInt(MaxSpawnRate * 0.032f), 2);
		// Burst
		MaxAPC += MaxBurstCount;
	}

	PeakActiveParticles = MaxAPC;

	return MaxAPC;
}


void UParticleLODLevel::ConvertToSpawnModule()
{
#if WITH_EDITOR
	// Move the required module SpawnRate and Burst information to a new SpawnModule.
	if (SpawnModule)
	{
//		UE_LOGF(LogParticles, Warning, "LOD Level already has a spawn module!");
		return;
	}

	UParticleEmitter* EmitterOuter = CastChecked<UParticleEmitter>(GetOuter());
	SpawnModule = NewObject<UParticleModuleSpawn>(EmitterOuter->GetOuter());
	check(SpawnModule);

	UDistributionFloat* SourceDist = RequiredModule->SpawnRate.Distribution;
	if (SourceDist)
	{
		SpawnModule->Rate.Distribution = Cast<UDistributionFloat>(StaticDuplicateObject(SourceDist, SpawnModule));
		SpawnModule->Rate.Distribution->bIsDirty = true;
		SpawnModule->Rate.Initialize();
	}

	// Now the burst list.
	int32 BurstCount = RequiredModule->BurstList.Num();
	if (BurstCount > 0)
	{
		SpawnModule->BurstList.AddZeroed(BurstCount);
		for (int32 BurstIndex = 0; BurstIndex < BurstCount; BurstIndex++)
		{
			SpawnModule->BurstList[BurstIndex].Count = RequiredModule->BurstList[BurstIndex].Count;
			SpawnModule->BurstList[BurstIndex].CountLow = RequiredModule->BurstList[BurstIndex].CountLow;
			SpawnModule->BurstList[BurstIndex].Time = RequiredModule->BurstList[BurstIndex].Time;
		}
	}

	MarkPackageDirty();
#endif	//#if WITH_EDITOR
}


int32 UParticleLODLevel::GetModuleIndex(UParticleModule* InModule)
{
	if (InModule)
	{
		if (InModule == RequiredModule)
		{
			return INDEX_REQUIREDMODULE;
		}
		else if (InModule == SpawnModule)
		{
			return INDEX_SPAWNMODULE;
		}
		else if (InModule == TypeDataModule)
		{
			return INDEX_TYPEDATAMODULE;
		}
		else
		{
			for (int32 ModuleIndex = 0; ModuleIndex < Modules.Num(); ModuleIndex++)
			{
				if (InModule == Modules[ModuleIndex])
				{
					return ModuleIndex;
				}
			}
		}
	}

	return INDEX_NONE;
}


UParticleModule* UParticleLODLevel::GetModuleAtIndex(int32 InIndex)
{
	// 'Normal' modules
	if (InIndex > INDEX_NONE)
	{
		if (InIndex < Modules.Num())
		{
			return Modules[InIndex];
		}

		return NULL;
	}

	switch (InIndex)
	{
	case INDEX_REQUIREDMODULE:		return RequiredModule;
	case INDEX_SPAWNMODULE:			return SpawnModule;
	case INDEX_TYPEDATAMODULE:		return TypeDataModule;
	}

	return NULL;
}


void UParticleLODLevel::SetLevelIndex(int32 InLevelIndex)
{
	// Remove the 'current' index from the validity flags and set the new one.
	RequiredModule->LODValidity &= ~(1 << Level);
	RequiredModule->LODValidity |= (1 << InLevelIndex);
	SpawnModule->LODValidity &= ~(1 << Level);
	SpawnModule->LODValidity |= (1 << InLevelIndex);
	if (TypeDataModule)
	{
		TypeDataModule->LODValidity &= ~(1 << Level);
		TypeDataModule->LODValidity |= (1 << InLevelIndex);
	}
	for (int32 ModuleIndex = 0; ModuleIndex < Modules.Num(); ModuleIndex++)
	{
		UParticleModule* CheckModule = Modules[ModuleIndex];
		if (CheckModule)
		{
			CheckModule->LODValidity &= ~(1 << Level);
			CheckModule->LODValidity |= (1 << InLevelIndex);
		}
	}

	Level = InLevelIndex;
}


bool UParticleLODLevel::IsModuleEditable(UParticleModule* InModule)
{
	// If the module validity flag is not set for this level, it is not editable.
	if ((InModule->LODValidity & (1 << Level)) == 0)
	{
		return false;
	}

	// If the module is shared w/ higher LOD levels, then it is not editable...
	int32 Validity = 0;
	if (Level > 0)
	{
		int32 Check = Level - 1;
		while (Check >= 0)
		{
			Validity |= (1 << Check);
			Check--;
		}

		if ((Validity & InModule->LODValidity) != 0)
		{
			return false;
		}
	}

	return true;
}

void UParticleLODLevel::GetUsedMaterials(TArray<UMaterialInterface*>& OutMaterials, const TArray<FNamedEmitterMaterial>& Slots, const TArray<class UMaterialInterface*>& EmitterMaterials) const
{
	// Only process enabled emitters
	if (bEnabled)
	{
		const UParticleModuleTypeDataMesh* MeshTypeData = Cast<UParticleModuleTypeDataMesh>(TypeDataModule);

		if (MeshTypeData && MeshTypeData->Mesh && MeshTypeData->Mesh->GetRenderData())
		{
			const FStaticMeshLODResources& LODModel = MeshTypeData->Mesh->GetRenderData()->LODResources[0];

			// Gather the materials applied to the LOD.
			for (int32 SectionIndex = 0; SectionIndex < LODModel.Sections.Num(); SectionIndex++)
			{
				UMaterialInterface* Material = NULL;
							
				TArray<FName>& NamedOverrides = RequiredModule->NamedMaterialOverrides;

				if (NamedOverrides.IsValidIndex(SectionIndex))
				{	
					//If we have named material overrides then get it's index into the emitter materials array.	
					for (int32 CheckIdx = 0; CheckIdx < Slots.Num(); ++CheckIdx)
					{
						if (NamedOverrides[SectionIndex] == Slots[CheckIdx].Name)
						{
							//Default to the default material for that slot.
							Material = Slots[CheckIdx].Material;
							if (EmitterMaterials.IsValidIndex(CheckIdx) && nullptr != EmitterMaterials[CheckIdx] )
							{
								//This material has been overridden externally, e.g. from a BP so use that one.
								Material = EmitterMaterials[CheckIdx];
							}

							break;
						}
					}
				}

				// See if there is a mesh material module.
				if (Material == NULL)
				{
					// Walk in reverse order as in the case of multiple modules, only the final result will be applied
					for (int32 ModuleIndex = Modules.Num()-1; ModuleIndex >= 0; --ModuleIndex)
					{
						UParticleModuleMeshMaterial* MeshMatModule = Cast<UParticleModuleMeshMaterial>(Modules[ModuleIndex]);
						if (MeshMatModule && MeshMatModule->bEnabled)
						{
							if (SectionIndex < MeshMatModule->MeshMaterials.Num())
							{
								Material = MeshMatModule->MeshMaterials[SectionIndex];
								break;
							}
						}
					}
				}

				// Overriding the material?
				if (Material == NULL && MeshTypeData->bOverrideMaterial == true)
				{
					Material = RequiredModule->Material;
				}

				// Use the material set on the mesh.
				if (Material == NULL)
				{
					Material = MeshTypeData->Mesh->GetMaterial(LODModel.Sections[SectionIndex].MaterialIndex);
				}

				if (Material)
				{
					OutMaterials.Add(Material);
				}
			}
		}
		else
		{
			UMaterialInterface* Material = NULL;
							
			TArray<FName>& NamedOverrides = RequiredModule->NamedMaterialOverrides;

			if (NamedOverrides.Num() > 0)
			{
				for (int32 CheckIdx = 0; CheckIdx < Slots.Num(); ++CheckIdx)
				{
					if (NamedOverrides[0] == Slots[CheckIdx].Name)
					{
						//Default to the default material for that slot.
						Material = Slots[CheckIdx].Material;
						if (EmitterMaterials.IsValidIndex(CheckIdx) && nullptr != EmitterMaterials[CheckIdx])
						{
							//This material has been overridden externally, e.g. from a BP so use that one.
							Material = EmitterMaterials[CheckIdx];
						}
        
						break;
					}
				}
			}

			if (!Material)
			{
				Material = RequiredModule->Material;
			}

			OutMaterials.Add(Material);
		}
	}
}

void UParticleLODLevel::GetStreamingMeshInfo(const FBoxSphereBounds& Bounds, TArray<FStreamingRenderAssetPrimitiveInfo>& OutStreamingRenderAssets) const
{
	if (bEnabled)
	{
		if (const UParticleModuleTypeDataMesh* MeshTypeData = Cast<UParticleModuleTypeDataMesh>(TypeDataModule))
		{
			if (UStaticMesh* Mesh = MeshTypeData->Mesh)
			{
				if (Mesh->RenderResourceSupportsStreaming() && Mesh->GetRenderAssetType() == EStreamableRenderAssetType::StaticMesh)
				{
					const FBoxSphereBounds MeshBounds = Mesh->GetBounds();
					const FBoxSphereBounds StreamingBounds = FBoxSphereBounds(
						Bounds.Origin + MeshBounds.Origin,
						MeshBounds.BoxExtent * MeshTypeData->LODSizeScale,
						MeshBounds.SphereRadius * MeshTypeData->LODSizeScale);
					const float MeshTexelFactor = MeshBounds.SphereRadius * 2.0f;

					new (OutStreamingRenderAssets) FStreamingRenderAssetPrimitiveInfo(Mesh, StreamingBounds, MeshTexelFactor);
				}
			}
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////
