// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	ParticleMemoryStatManager.cpp: Particle dynamic data memory manager implementation.
=============================================================================*/

#include "EngineStats.h"
#include "ParticleHelper.h"

/*-----------------------------------------------------------------------------
	ParticleMemoryStatManager
-----------------------------------------------------------------------------*/

DEFINE_STAT(STAT_ParticleManagerUpdateData);

#if STATS
uint32 FParticleMemoryStatManager::DynamicPSysCompCount = 0;
uint32 FParticleMemoryStatManager::DynamicPSysCompMem = 0;
uint32 FParticleMemoryStatManager::DynamicEmitterCount = 0;
uint32 FParticleMemoryStatManager::DynamicEmitterMem = 0;
uint32 FParticleMemoryStatManager::TotalGTParticleData = 0;
uint32 FParticleMemoryStatManager::TotalRTParticleData = 0;
uint32 FParticleMemoryStatManager::DynamicSpriteCount = 0;
uint32 FParticleMemoryStatManager::DynamicSubUVCount = 0;
uint32 FParticleMemoryStatManager::DynamicMeshCount = 0;
uint32 FParticleMemoryStatManager::DynamicBeamCount = 0;
uint32 FParticleMemoryStatManager::DynamicRibbonCount = 0;
uint32 FParticleMemoryStatManager::DynamicAnimTrailCount = 0;
uint32 FParticleMemoryStatManager::DynamicSpriteGTMem = 0;
uint32 FParticleMemoryStatManager::DynamicSubUVGTMem = 0;
uint32 FParticleMemoryStatManager::DynamicMeshGTMem = 0;
uint32 FParticleMemoryStatManager::DynamicBeamGTMem = 0;
uint32 FParticleMemoryStatManager::DynamicRibbonGTMem = 0;
uint32 FParticleMemoryStatManager::DynamicAnimTrailGTMem = 0;
uint32 FParticleMemoryStatManager::DynamicUntrackedGTMem = 0;
uint32 FParticleMemoryStatManager::DynamicPSysCompCount_MAX = 0;
uint32 FParticleMemoryStatManager::DynamicPSysCompMem_MAX = 0;
uint32 FParticleMemoryStatManager::DynamicEmitterCount_MAX = 0;
uint32 FParticleMemoryStatManager::DynamicEmitterMem_MAX = 0;
uint32 FParticleMemoryStatManager::DynamicEmitterGTMem_Waste_MAX = 0;
uint32 FParticleMemoryStatManager::DynamicEmitterGTMem_Largest_MAX = 0;
uint32 FParticleMemoryStatManager::TotalGTParticleData_MAX = 0;
uint32 FParticleMemoryStatManager::TotalRTParticleData_MAX = 0;
uint32 FParticleMemoryStatManager::LargestRTParticleData_MAX = 0;
uint32 FParticleMemoryStatManager::DynamicSpriteCount_MAX = 0;
uint32 FParticleMemoryStatManager::DynamicSubUVCount_MAX = 0;
uint32 FParticleMemoryStatManager::DynamicMeshCount_MAX = 0;
uint32 FParticleMemoryStatManager::DynamicBeamCount_MAX = 0;
uint32 FParticleMemoryStatManager::DynamicRibbonCount_MAX = 0;
uint32 FParticleMemoryStatManager::DynamicAnimTrailCount_MAX = 0;
uint32 FParticleMemoryStatManager::DynamicSpriteGTMem_MAX = 0;
uint32 FParticleMemoryStatManager::DynamicSubUVGTMem_MAX = 0;
uint32 FParticleMemoryStatManager::DynamicMeshGTMem_MAX = 0;
uint32 FParticleMemoryStatManager::DynamicBeamGTMem_MAX = 0;
uint32 FParticleMemoryStatManager::DynamicRibbonGTMem_MAX = 0;
uint32 FParticleMemoryStatManager::DynamicAnimTrailGTMem_MAX = 0;
uint32 FParticleMemoryStatManager::DynamicUntrackedGTMem_MAX = 0;

void FParticleMemoryStatManager::ResetParticleMemoryMaxValues()
{
	DynamicPSysCompCount_MAX = 0;
	DynamicPSysCompMem_MAX = 0;
	DynamicEmitterCount_MAX = 0;
	DynamicEmitterMem_MAX = 0;
	DynamicEmitterGTMem_Waste_MAX = 0;
	DynamicEmitterGTMem_Largest_MAX = 0;
	TotalGTParticleData_MAX = 0;
	TotalRTParticleData_MAX = 0;
	LargestRTParticleData_MAX = 0;
	DynamicSpriteCount_MAX = 0;
	DynamicSubUVCount_MAX = 0;
	DynamicMeshCount_MAX = 0;
	DynamicBeamCount_MAX = 0;
	DynamicRibbonCount_MAX = 0;
	DynamicAnimTrailCount_MAX = 0;
	DynamicSpriteGTMem_MAX = 0;
	DynamicSubUVGTMem_MAX = 0;
	DynamicMeshGTMem_MAX = 0;
	DynamicBeamGTMem_MAX = 0;
	DynamicRibbonGTMem_MAX = 0;
	DynamicAnimTrailGTMem_MAX = 0;
	DynamicUntrackedGTMem_MAX = 0;
}

void FParticleMemoryStatManager::DumpParticleMemoryStats(FOutputDevice& Ar)
{
	// 
	Ar.Logf(TEXT("Particle Dynamic Memory Stats"));

	Ar.Logf(TEXT("Type,Count,MaxCount,Mem(Bytes),MaxMem(Bytes),GTMem(Bytes),GTMemMax(Bytes)"));
	Ar.Logf(TEXT("Total PSysComponents,%d,%d,%d,%d,%d,%d"), 
		DynamicPSysCompCount, DynamicPSysCompCount_MAX, 
		DynamicPSysCompMem,DynamicPSysCompMem_MAX,
		0,0);
	Ar.Logf(TEXT("Total DynamicEmitters,%d,%d,%d,%d,%d,%d"), 
		DynamicEmitterCount, DynamicEmitterCount_MAX, 
		DynamicEmitterMem, DynamicEmitterMem_MAX,
		TotalGTParticleData, TotalGTParticleData_MAX);
	Ar.Logf(TEXT("Sprite,%d,%d,%d,%d,%d,%d"), 
		DynamicSpriteCount, DynamicSpriteCount_MAX, 
		DynamicSpriteCount * sizeof(FDynamicSpriteEmitterData), 
		DynamicSpriteCount_MAX * sizeof(FDynamicSpriteEmitterData), 
		DynamicSpriteGTMem, DynamicSpriteGTMem_MAX);
	Ar.Logf(TEXT("Mesh,%d,%d,%d,%d,%d,%d"), 
		DynamicMeshCount, DynamicMeshCount_MAX, 
		DynamicMeshCount * sizeof(FDynamicMeshEmitterData), 
		DynamicMeshCount_MAX * sizeof(FDynamicMeshEmitterData), 
		DynamicMeshGTMem, DynamicMeshGTMem_MAX);
	Ar.Logf(TEXT("Beam,%d,%d,%d,%d,%d,%d"), 
		DynamicBeamCount, DynamicBeamCount_MAX, 
		DynamicBeamCount * sizeof(FDynamicBeam2EmitterData), 
		DynamicBeamCount_MAX * sizeof(FDynamicBeam2EmitterData), 
		DynamicBeamGTMem, DynamicBeamGTMem_MAX);
	Ar.Logf(TEXT("Ribbon,%d,%d,%d,%d,%d,%d"), 
		DynamicRibbonCount, DynamicRibbonCount_MAX, 
		DynamicRibbonCount * sizeof(FDynamicRibbonEmitterData), 
		DynamicRibbonCount_MAX * sizeof(FDynamicRibbonEmitterData), 
		DynamicRibbonGTMem, DynamicRibbonGTMem_MAX);
	Ar.Logf(TEXT("AnimTrail,%d,%d,%d,%d,%d,%d"), 
		DynamicAnimTrailCount, DynamicAnimTrailCount_MAX, 
		DynamicAnimTrailCount * sizeof(FDynamicAnimTrailEmitterData), 
		DynamicAnimTrailCount_MAX * sizeof(FDynamicAnimTrailEmitterData), 
		DynamicAnimTrailGTMem, DynamicAnimTrailGTMem_MAX);
	Ar.Logf(TEXT("Untracked,%d,%d,%d,%d,%d,%d"), 0, 0, 0, 0, DynamicUntrackedGTMem, DynamicUntrackedGTMem_MAX);

	Ar.Logf(TEXT("ParticleData,Total(Bytes),FMath::Max(Bytes)"));
	Ar.Logf(TEXT("GameThread,%d,%d"), TotalGTParticleData, TotalGTParticleData_MAX);
	Ar.Logf(TEXT("RenderThread,%d,%d"), TotalRTParticleData, TotalRTParticleData_MAX);

	Ar.Logf(TEXT("Max wasted GT,%d"), DynamicEmitterGTMem_Waste_MAX);
	Ar.Logf(TEXT("Largest single GT allocation,%d"), DynamicEmitterGTMem_Largest_MAX);
	Ar.Logf(TEXT("Largest single RT allocation,%d"), LargestRTParticleData_MAX);
}

/**
 *	Update the stats for all particle system components
 */
void FParticleMemoryStatManager::UpdateStats()
{
#if 0 // this is a visualization thing, not a stats thing
	SCOPE_CYCLE_COUNTER(STAT_ParticleManagerUpdateData);
	// Log/setup/etc.
	DynamicPSysCompCount = GET_DWORD_STAT(STAT_DynamicPSysCompCount);
	DynamicPSysCompMem = GET_DWORD_STAT(STAT_DynamicPSysCompMem);
	DynamicEmitterCount = GET_DWORD_STAT(STAT_DynamicEmitterCount);
	DynamicEmitterMem = GET_DWORD_STAT(STAT_DynamicEmitterMem);
	TotalGTParticleData = GET_DWORD_STAT(STAT_GTParticleData);
	TotalRTParticleData = GET_DWORD_STAT(STAT_RTParticleData);
	uint32 LargestRTParticleData = GET_DWORD_STAT(STAT_RTParticleData_Largest);
	DynamicSpriteCount = GET_DWORD_STAT(STAT_DynamicSpriteCount);
	DynamicSubUVCount = GET_DWORD_STAT(STAT_DynamicSubUVCount);
	DynamicMeshCount = GET_DWORD_STAT(STAT_DynamicMeshCount);
	DynamicBeamCount = GET_DWORD_STAT(STAT_DynamicBeamCount);
	DynamicRibbonCount = GET_DWORD_STAT(STAT_DynamicRibbonCount);
	DynamicAnimTrailCount = GET_DWORD_STAT(STAT_DynamicAnimTrailCount);

	DynamicSpriteGTMem = GET_DWORD_STAT(STAT_DynamicSpriteGTMem);
	DynamicSubUVGTMem = GET_DWORD_STAT(STAT_DynamicSubUVGTMem);
	DynamicMeshGTMem = GET_DWORD_STAT(STAT_DynamicMeshGTMem);
	DynamicBeamGTMem = GET_DWORD_STAT(STAT_DynamicBeamGTMem);
	DynamicRibbonGTMem = GET_DWORD_STAT(STAT_DynamicRibbonGTMem);
	DynamicAnimTrailGTMem = GET_DWORD_STAT(STAT_DynamicAnimTrailGTMem);
	DynamicUntrackedGTMem = GET_DWORD_STAT(STAT_DynamicUntrackedGTMem);

	uint32 DynamicEmitterGTMem_Waste = GET_DWORD_STAT(STAT_DynamicEmitterGTMem_Waste);
	uint32 DynamicEmitterGTMem_Largest = GET_DWORD_STAT(STAT_DynamicEmitterGTMem_Largest);
	DynamicEmitterGTMem_Waste_MAX = FMath::Max<uint32>(DynamicEmitterGTMem_Waste_MAX, DynamicEmitterGTMem_Waste);
	DynamicEmitterGTMem_Largest_MAX = FMath::Max<uint32>(DynamicEmitterGTMem_Largest_MAX, DynamicEmitterGTMem_Largest);

	DynamicPSysCompCount_MAX = FMath::Max<uint32>(DynamicPSysCompCount_MAX, DynamicPSysCompCount);
	DynamicPSysCompMem_MAX = FMath::Max<uint32>(DynamicPSysCompMem_MAX, DynamicPSysCompMem);
	DynamicEmitterCount_MAX = FMath::Max<uint32>(DynamicEmitterCount_MAX, DynamicEmitterCount);
	DynamicEmitterMem_MAX = FMath::Max<uint32>(DynamicEmitterMem_MAX, DynamicEmitterMem);
	TotalGTParticleData_MAX = FMath::Max<uint32>(TotalGTParticleData_MAX, TotalGTParticleData);
	TotalRTParticleData_MAX = FMath::Max<uint32>(TotalRTParticleData_MAX, TotalRTParticleData);
	LargestRTParticleData_MAX = FMath::Max<uint32>(LargestRTParticleData_MAX, LargestRTParticleData);

	DynamicSpriteCount_MAX = FMath::Max<uint32>(DynamicSpriteCount_MAX, DynamicSpriteCount);
	DynamicSubUVCount_MAX = FMath::Max<uint32>(DynamicSubUVCount_MAX, DynamicSubUVCount);
	DynamicMeshCount_MAX = FMath::Max<uint32>(DynamicMeshCount_MAX, DynamicMeshCount);
	DynamicBeamCount_MAX = FMath::Max<uint32>(DynamicBeamCount_MAX, DynamicBeamCount);
	DynamicRibbonCount_MAX = FMath::Max<uint32>(DynamicRibbonCount_MAX, DynamicRibbonCount);
	DynamicAnimTrailCount_MAX = FMath::Max<uint32>(DynamicAnimTrailCount_MAX, DynamicAnimTrailCount);

	DynamicSpriteGTMem_MAX = FMath::Max<uint32>(DynamicSpriteGTMem_MAX, DynamicSpriteGTMem);
	DynamicSubUVGTMem_MAX = FMath::Max<uint32>(DynamicSubUVGTMem_MAX, DynamicSubUVGTMem);
	DynamicMeshGTMem_MAX = FMath::Max<uint32>(DynamicMeshGTMem_MAX, DynamicMeshGTMem);
	DynamicBeamGTMem_MAX = FMath::Max<uint32>(DynamicBeamGTMem_MAX, DynamicBeamGTMem);
	DynamicRibbonGTMem_MAX = FMath::Max<uint32>(DynamicRibbonGTMem_MAX, DynamicRibbonGTMem);
	DynamicAnimTrailGTMem_MAX = FMath::Max<uint32>(DynamicAnimTrailGTMem_MAX, DynamicAnimTrailGTMem);
	DynamicUntrackedGTMem_MAX = FMath::Max<uint32>(DynamicUntrackedGTMem_MAX, DynamicUntrackedGTMem);

	GStatManager.SetStatValue(STAT_DynamicPSysCompCount_MAX, DynamicPSysCompCount_MAX);
	GStatManager.SetStatValue(STAT_DynamicPSysCompMem_MAX, DynamicPSysCompMem_MAX);
	GStatManager.SetStatValue(STAT_DynamicEmitterCount_MAX, DynamicEmitterCount_MAX);
	GStatManager.SetStatValue(STAT_DynamicEmitterMem_MAX, DynamicEmitterMem_MAX);
	GStatManager.SetStatValue(STAT_DynamicEmitterGTMem_Waste_MAX, DynamicEmitterGTMem_Waste_MAX);
	GStatManager.SetStatValue(STAT_DynamicEmitterGTMem_Largest_MAX, DynamicEmitterGTMem_Largest_MAX);
	GStatManager.SetStatValue(STAT_GTParticleData_MAX, TotalGTParticleData_MAX);
	GStatManager.SetStatValue(STAT_RTParticleData_MAX, TotalRTParticleData_MAX);
	GStatManager.SetStatValue(STAT_RTParticleData_Largest_MAX, LargestRTParticleData_MAX);

	GStatManager.SetStatValue(STAT_DynamicSpriteCount_MAX, DynamicSpriteCount_MAX);
	GStatManager.SetStatValue(STAT_DynamicSubUVCount_MAX, DynamicSubUVCount_MAX);
	GStatManager.SetStatValue(STAT_DynamicMeshCount_MAX, DynamicMeshCount_MAX);
	GStatManager.SetStatValue(STAT_DynamicBeamCount_MAX, DynamicBeamCount_MAX);
	GStatManager.SetStatValue(STAT_DynamicRibbonCount_MAX, DynamicRibbonCount_MAX);
	GStatManager.SetStatValue(STAT_DynamicAnimTrailCount_MAX, DynamicAnimTrailCount_MAX);

	GStatManager.SetStatValue(STAT_DynamicSpriteGTMem_MAX, DynamicSpriteGTMem_MAX);
	GStatManager.SetStatValue(STAT_DynamicSubUVGTMem_Max, DynamicSubUVGTMem_MAX);
	GStatManager.SetStatValue(STAT_DynamicMeshGTMem_MAX, DynamicMeshGTMem_MAX);
	GStatManager.SetStatValue(STAT_DynamicBeamGTMem_MAX, DynamicBeamGTMem_MAX);
	GStatManager.SetStatValue(STAT_DynamicRibbonGTMem_MAX, DynamicRibbonGTMem_MAX);
	GStatManager.SetStatValue(STAT_DynamicAnimTrailGTMem_MAX, DynamicAnimTrailGTMem_MAX);
	GStatManager.SetStatValue(STAT_DynamicUntrackedGTMem_MAX, DynamicUntrackedGTMem_MAX);
#endif
}
#endif
