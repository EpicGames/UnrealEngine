// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	ParticleModules.cpp: Particle module implementation.
=============================================================================*/

#include "Distributions/DistributionFloatParameterBase.h"
#include "UObject/RenderingObjectVersion.h"
#include "Distributions/DistributionVectorParameterBase.h"
#include "UObject/Package.h"
#include "GameFramework/WorldSettings.h"
#include "Math/InterpCurve.h"
#include "Particles/ParticleSystem.h"
#include "Distributions/Distribution.h"
#include "MeshParticleVertexFactory.h"
#include "Particles/ParticleModule.h"
#include "Particles/Location/ParticleModuleLocationBase.h"
#include "Particles/Orientation/ParticleModuleOrientationBase.h"
#include "Particles/Orientation/ParticleModuleOrientationAxisLock.h"
#include "ParticleEmitterInstances.h"
#include "ParticleEmitterInstanceOwner.h"
#include "Particles/ParticleSpriteEmitter.h"
#include "Particles/ParticleSystemComponent.h"
#include "Distributions/DistributionFloatConstant.h"
#include "Distributions/DistributionFloatUniform.h"
#include "Distributions/DistributionVectorConstant.h"
#include "Distributions/DistributionVectorUniform.h"
#include "Engine/StaticMesh.h"
#include "Engine/Texture2D.h"
#include "SceneInterface.h"
#include "UnrealEngine.h"
#include "Distributions/DistributionFloatParticleParameter.h"
#include "Distributions/DistributionVectorParticleParameter.h"
#include "Distributions/DistributionVectorConstantCurve.h"
#include "Distributions/DistributionVectorUniformCurve.h"
#include "Particles/Acceleration/ParticleModuleAccelerationBase.h"
#include "Particles/Acceleration/ParticleModuleAcceleration.h"
#include "Particles/Acceleration/ParticleModuleAccelerationConstant.h"
#include "Particles/Acceleration/ParticleModuleAccelerationDrag.h"
#include "Particles/Acceleration/ParticleModuleAccelerationDragScaleOverLife.h"
#include "Particles/Acceleration/ParticleModuleAccelerationOverLifetime.h"
#include "Particles/Attractor/ParticleModuleAttractorBase.h"
#include "Particles/Attractor/ParticleModuleAttractorPointGravity.h"
#include "Particles/Attractor/ParticleModuleAttractorLine.h"
#include "Particles/Attractor/ParticleModuleAttractorParticle.h"
#include "Particles/Attractor/ParticleModuleAttractorPoint.h"
#include "Particles/Kill/ParticleModuleKillBase.h"
#include "Particles/Kill/ParticleModuleKillBox.h"
#include "Particles/Kill/ParticleModuleKillHeight.h"
#include "Particles/Light/ParticleModuleLightBase.h"
#include "Particles/Light/ParticleModuleLight.h"
#include "Particles/Light/ParticleModuleLight_Seeded.h"
#include "Particles/Lifetime/ParticleModuleLifetimeBase.h"
#include "Particles/Lifetime/ParticleModuleLifetime.h"
#include "Particles/Lifetime/ParticleModuleLifetime_Seeded.h"
#include "Particles/Location/ParticleModuleSourceMovement.h"
#include "Particles/Modules/Location/ParticleModulePivotOffset.h"
#include "Particles/Rotation/ParticleModuleRotationBase.h"
#include "Particles/Rotation/ParticleModuleRotation.h"
#include "Particles/Rotation/ParticleModuleRotation_Seeded.h"
#include "Particles/Rotation/ParticleModuleMeshRotation.h"
#include "Particles/Rotation/ParticleModuleMeshRotation_Seeded.h"
#include "Particles/Rotation/ParticleModuleRotationOverLifetime.h"
#include "Particles/RotationRate/ParticleModuleRotationRateBase.h"
#include "Particles/RotationRate/ParticleModuleRotationRate.h"
#include "Particles/RotationRate/ParticleModuleRotationRateMultiplyLife.h"
#include "Particles/RotationRate/ParticleModuleMeshRotationRate.h"
#include "Particles/RotationRate/ParticleModuleRotationRate_Seeded.h"
#include "Particles/RotationRate/ParticleModuleMeshRotationRate_Seeded.h"
#include "Particles/RotationRate/ParticleModuleMeshRotationRateMultiplyLife.h"
#include "Particles/RotationRate/ParticleModuleMeshRotationRateOverLife.h"
#include "Particles/SubUV/ParticleModuleSubUVBase.h"
#include "Particles/ParticleEmitter.h"
#include "Particles/SubUVAnimation.h"
#include "Particles/SubUV/ParticleModuleSubUV.h"
#include "Particles/SubUV/ParticleModuleSubUVMovie.h"
#include "Particles/TypeData/ParticleModuleTypeDataBase.h"
#include "Particles/TypeData/ParticleModuleTypeDataGpu.h"
#include "Particles/TypeData/ParticleModuleTypeDataMesh.h"
#include "Particles/ParticleLODLevel.h"
#include "Particles/ParticleModuleRequired.h"
#include "Particles/FXSystemPrivate.h"
#include "Distributions/DistributionFloatUniformCurve.h"
#include "Engine/InterpCurveEdSetup.h"
#include "Engine/RendererSettings.h"
#include "Distributions/DistributionFloatConstantCurve.h"
#include "Components/PointLightComponent.h"
#include "Particles/Collision/ParticleModuleCollisionGPU.h"
#include "DerivedDataCacheInterface.h"
#include "StaticMeshResources.h"

static TAutoConsoleVariable<bool> CVarFxCascadeUseVelocityForMotionBlur(
	TEXT("fx.Cascade.UseVelocityForMotionBlur"),
	true,
	TEXT("When enabled velocity will be used to approximate velocity for vertex factories that support this.")
);

/*-----------------------------------------------------------------------------
	Abstract base modules used for categorization.
-----------------------------------------------------------------------------*/

/*-----------------------------------------------------------------------------
	Helper functions.
-----------------------------------------------------------------------------*/

/*-----------------------------------------------------------------------------
	UParticleModule implementation.
-----------------------------------------------------------------------------*/
UParticleModule::UParticleModule(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bSupported3DDrawMode = false;
	b3DDrawMode = false;
	bEnabled = true;
	bEditable = true;
	LODDuplicate = true;
	bSupportsRandomSeed = false;
	bRequiresLoopingNotification = false;
	bUpdateForGPUEmitter = false;
}

bool UParticleModule::IsPostLoadThreadSafe() const
{
	return true;
}

#if WITH_EDITOR
void UParticleModule::PostEditChangeProperty( FPropertyChangedEvent& PropertyChangedEvent )
{
	Super::PostEditChangeProperty( PropertyChangedEvent );

	// Rebuild emitters for this particle system.
	UParticleSystem* ParticleSystem = CastChecked<UParticleSystem>( GetOuter() );
	ParticleSystem->BuildEmitters();
}
#endif // WITH_EDITOR

void UParticleModule::CompileModule( FParticleEmitterBuildInfo& EmitterInfo )
{
	if ( bSpawnModule )
	{
		EmitterInfo.SpawnModules.Add( this );
	}
}

void UParticleModule::Spawn(const FSpawnContext& Context)
{
}

void UParticleModule::Update(const FUpdateContext& Context)
{
}


void UParticleModule::FinalUpdate(const FUpdateContext& Context)
{
}

uint32 UParticleModule::RequiredBytes(UParticleModuleTypeDataBase* TypeData)
{
	return 0;
}

uint32 UParticleModule::RequiredBytesPerInstance()
{
	return 0;
}

uint32 UParticleModule::PrepPerInstanceBlock(FParticleEmitterInstance* Owner, void* InstData)
{
	return 0xffffffff;
}


void UParticleModule::SetToSensibleDefaults(UParticleEmitter* Owner)
{
	// The default implementation does nothing...
}

void UParticleModule::GetCurveObjects(TArray<FParticleCurvePair>& OutCurves)
{
	for (TFieldIterator<FProperty> It(GetClass()); It; ++It)
	{
		UObject* Distribution = NULL;
		FProperty* Property = *It;
		check(Property != NULL);

		// attempt to get a distribution from a random struct property
		if (Property->IsA(FStructProperty::StaticClass()))
		{
			Distribution = FRawDistribution::TryGetDistributionObjectFromRawDistributionProperty((FStructProperty*)Property, (uint8*)this);
		}
		else if (Property->IsA(FObjectPropertyBase::StaticClass()))
		{
			FObjectPropertyBase* ObjProperty = (FObjectPropertyBase*)(Property);
			if (ObjProperty && (ObjProperty->PropertyClass == UDistributionFloat::StaticClass() ||
				ObjProperty->PropertyClass == UDistributionVector::StaticClass()))
			{
				Distribution = ObjProperty->GetObjectPropertyValue(ObjProperty->ContainerPtrToValuePtr<void>(this));
			}
		}

		if (Distribution)
		{
			FParticleCurvePair* NewCurve = new(OutCurves)FParticleCurvePair;
			check(NewCurve);
			NewCurve->CurveObject = Distribution;
			NewCurve->CurveName = It->GetName();
		}
	}
}

bool UParticleModule::AddModuleCurvesToEditor(UInterpCurveEdSetup* EdSetup, TArray<const FCurveEdEntry*>& OutCurveEntries)
{
	bool bNewCurve = false;
#if WITH_EDITORONLY_DATA
	TArray<FParticleCurvePair> OutCurves;
	GetCurveObjects(OutCurves);
	for (int32 CurveIndex = 0; CurveIndex < OutCurves.Num(); CurveIndex++)
	{
		UObject* Distribution = OutCurves[CurveIndex].CurveObject;
		if (Distribution)
		{
			FCurveEdEntry* Curve = NULL;
			bNewCurve |= EdSetup->AddCurveToCurrentTab(Distribution, OutCurves[CurveIndex].CurveName, ModuleEditorColor, &Curve, bCurvesAsColor, bCurvesAsColor);
			OutCurveEntries.Add( Curve );		
		}
	}
#endif // WITH_EDITORONLY_DATA
	return bNewCurve;
}

void UParticleModule::RemoveModuleCurvesFromEditor(UInterpCurveEdSetup* EdSetup)
{
	TArray<FParticleCurvePair> OutCurves;
	GetCurveObjects(OutCurves);
	for (int32 CurveIndex = 0; CurveIndex < OutCurves.Num(); CurveIndex++)
	{
		UObject* Distribution = OutCurves[CurveIndex].CurveObject;
		if (Distribution)
		{
			EdSetup->RemoveCurve(Distribution);
		}
	}
}

bool UParticleModule::ModuleHasCurves()
{
	TArray<FParticleCurvePair> Curves;
	GetCurveObjects(Curves);

	return (Curves.Num() > 0);
}

bool UParticleModule::IsDisplayedInCurveEd(UInterpCurveEdSetup* EdSetup)
{
	TArray<FParticleCurvePair> Curves;
	GetCurveObjects(Curves);

	for(int32 i=0; i<Curves.Num(); i++)
	{
		if(EdSetup->ShowingCurve(Curves[i].CurveObject))
		{
			return true;
		}	
	}

	return false;
}

void UParticleModule::ChangeEditorColor(FColor& Color, UInterpCurveEdSetup* EdSetup)
{
#if WITH_EDITORONLY_DATA
	ModuleEditorColor	= Color;

	TArray<FParticleCurvePair> Curves;
	GetCurveObjects(Curves);

	for (int32 TabIndex = 0; TabIndex < EdSetup->Tabs.Num(); TabIndex++)
	{
		FCurveEdTab*	Tab = &(EdSetup->Tabs[TabIndex]);
		for (int32 CurveIndex = 0; CurveIndex < Tab->Curves.Num(); CurveIndex++)
		{
			FCurveEdEntry* Entry	= &(Tab->Curves[CurveIndex]);
			for (int32 MyCurveIndex = 0; MyCurveIndex < Curves.Num(); MyCurveIndex++)
			{
				if (Curves[MyCurveIndex].CurveObject == Entry->CurveObject)
				{
					Entry->CurveColor	= Color;
				}
			}
		}
	}
#endif // WITH_EDITORONLY_DATA
}

void UParticleModule::AutoPopulateInstanceProperties(UParticleSystemComponent* PSysComp)
{
	check(IsInGameThread());
	for (TFieldIterator<FStructProperty> It(GetClass()); It; ++It)
	{
		// attempt to get a distribution from a random struct property
		UObject* Distribution = FRawDistribution::TryGetDistributionObjectFromRawDistributionProperty(*It, (uint8*)this);
		if (Distribution)
		{
			EParticleSysParamType ParamType	= PSPT_None;
			FName ParamName;

			// only handle particle param types
			UDistributionFloatParticleParameter* DistFloatParam = Cast<UDistributionFloatParticleParameter>(Distribution);
			UDistributionVectorParticleParameter* DistVectorParam = Cast<UDistributionVectorParticleParameter>(Distribution);
			if (DistFloatParam != NULL)
			{
				ParamType = PSPT_Scalar;
				ParamName = DistFloatParam->ParameterName;
				
			}
			else if (DistVectorParam != NULL)
			{
				ParamType = PSPT_Vector;
				ParamName = DistVectorParam->ParameterName;
			}

			if (ParamType != PSPT_None)
			{
				bool	bFound	= false;
				for (int32 i = 0; i < PSysComp->InstanceParameters.Num(); i++)
				{
					FParticleSysParam* Param = &(PSysComp->InstanceParameters[i]);
					
					if (Param->Name == ParamName)
					{
						bFound	=	true;
						break;
					}
				}

				if (!bFound)
				{
					int32 NewParamIndex = PSysComp->InstanceParameters.AddZeroed();
					PSysComp->InstanceParameters[NewParamIndex].Name		= ParamName;
					PSysComp->InstanceParameters[NewParamIndex].ParamType	= ParamType;
					PSysComp->InstanceParameters[NewParamIndex].Actor		= NULL;
					// Populate a Vector or Scalar using GetValue. (If we just call GetValue with no parameters we will get the default value based on the setting of the Parameter)
					switch (ParamType)
					{
					case PSPT_Vector:
						PSysComp->InstanceParameters[NewParamIndex].Vector = DistVectorParam->GetValue();
						PSysComp->InstanceParameters[NewParamIndex].Vector_Low = DistVectorParam->MinOutput;
						break;
					case PSPT_Scalar:
						PSysComp->InstanceParameters[NewParamIndex].Scalar = DistFloatParam->GetValue();
						PSysComp->InstanceParameters[NewParamIndex].Scalar_Low = DistFloatParam->MinOutput;
						break;
					}
				}
			}
		}
	}

	FParticleRandomSeedInfo* SeedInfo = GetRandomSeedInfo();
	if (SeedInfo != NULL)
	{
		if (SeedInfo->ParameterName != NAME_None)
		{
			bool	bFound	= false;
			for (int32 ParamIdx = 0; ParamIdx < PSysComp->InstanceParameters.Num(); ParamIdx++)
			{
				FParticleSysParam* Param = &(PSysComp->InstanceParameters[ParamIdx]);
				if (Param->Name == SeedInfo->ParameterName)
				{
					bFound = true;
					break;
				}
			}

			if (bFound == false)
			{
				int32 NewParamIndex = PSysComp->InstanceParameters.AddZeroed();
				PSysComp->InstanceParameters[NewParamIndex].Name = SeedInfo->ParameterName;
				PSysComp->InstanceParameters[NewParamIndex].ParamType = PSPT_Scalar;
			}
		}
	}
}


bool UParticleModule::GenerateLODModuleValues(UParticleModule* SourceModule, float Percentage, UParticleLODLevel* LODLevel)
{
	return true;
}


bool UParticleModule::ConvertFloatDistribution(UDistributionFloat* FloatDist, UDistributionFloat* SourceFloatDist, float Percentage)
{
	float	Multiplier	= Percentage / 100.0f;

	UDistributionFloatConstant*				DistConstant		= Cast<UDistributionFloatConstant>(FloatDist);
	UDistributionFloatConstantCurve*		DistConstantCurve	= Cast<UDistributionFloatConstantCurve>(FloatDist);
	UDistributionFloatUniform*				DistUniform			= Cast<UDistributionFloatUniform>(FloatDist);
	UDistributionFloatUniformCurve*			DistUniformCurve	= Cast<UDistributionFloatUniformCurve>(FloatDist);
	UDistributionFloatParticleParameter*	DistParticleParam	= Cast<UDistributionFloatParticleParameter>(FloatDist);

	if (DistParticleParam)
	{
		DistParticleParam->MinOutput	*= Multiplier;
		DistParticleParam->MaxOutput	*= Multiplier;
	}
	else
	if (DistUniformCurve)
	{
		for (int32 KeyIndex = 0; KeyIndex < DistUniformCurve->GetNumKeys(); KeyIndex++)
		{
			for (int32 SubIndex = 0; SubIndex < DistUniformCurve->GetNumSubCurves(); SubIndex++)
			{
				float	Value	= DistUniformCurve->GetKeyOut(SubIndex, KeyIndex);
				DistUniformCurve->SetKeyOut(SubIndex, KeyIndex, Value * Multiplier);
			}
		}
	}
	else
	if (DistConstant)
	{
		UDistributionFloatConstant*	SourceConstant	= Cast<UDistributionFloatConstant>(SourceFloatDist);
		check(SourceConstant);
		DistConstant->SetKeyOut(0, 0, SourceConstant->Constant * Multiplier);
	}
	else
	if (DistConstantCurve)
	{
		UDistributionFloatConstantCurve* SourceConstantCurve	= Cast<UDistributionFloatConstantCurve>(SourceFloatDist);
		check(SourceConstantCurve);

		for (int32 KeyIndex = 0; KeyIndex < SourceConstantCurve->GetNumKeys(); KeyIndex++)
		{
			DistConstantCurve->CreateNewKey(SourceConstantCurve->GetKeyIn(KeyIndex));
			for (int32 SubIndex = 0; SubIndex < SourceConstantCurve->GetNumSubCurves(); SubIndex++)
			{
				float	Value	= SourceConstantCurve->GetKeyOut(SubIndex, KeyIndex);
				DistConstantCurve->SetKeyOut(SubIndex, KeyIndex, Value * Multiplier);
			}
		}
	}
	else
	if (DistUniform)
	{
		DistUniform->SetKeyOut(0, 0, DistUniform->Min * Multiplier);
		DistUniform->SetKeyOut(1, 0, DistUniform->Max * Multiplier);
	}
	else
	{
		UE_LOGF(LogParticles, Log, "UParticleModule::ConvertFloatDistribution> Invalid distribution?");
		return false;
	}

	// Safety catch to ensure that the distribution lookup tables get rebuilt...
	FloatDist->bIsDirty = true;
	return true;
}


bool UParticleModule::ConvertVectorDistribution(UDistributionVector* VectorDist, UDistributionVector* SourceVectorDist, float Percentage)
{
	float	Multiplier	= Percentage / 100.0f;

	UDistributionVectorConstant*			DistConstant		= Cast<UDistributionVectorConstant>(VectorDist);
	UDistributionVectorConstantCurve*		DistConstantCurve	= Cast<UDistributionVectorConstantCurve>(VectorDist);
	UDistributionVectorUniform*				DistUniform			= Cast<UDistributionVectorUniform>(VectorDist);
	UDistributionVectorUniformCurve*		DistUniformCurve	= Cast<UDistributionVectorUniformCurve>(VectorDist);
	UDistributionVectorParticleParameter*	DistParticleParam	= Cast<UDistributionVectorParticleParameter>(VectorDist);

	if (DistParticleParam)
	{
		DistParticleParam->MinOutput.X	*= Multiplier;
		DistParticleParam->MinOutput.Y	*= Multiplier;
		DistParticleParam->MinOutput.Z	*= Multiplier;
		DistParticleParam->MaxOutput.X	*= Multiplier;
		DistParticleParam->MaxOutput.Y	*= Multiplier;
		DistParticleParam->MaxOutput.Z	*= Multiplier;
	}
	else
	if (DistUniformCurve)
	{
		for (int32 KeyIndex = 0; KeyIndex < DistUniformCurve->GetNumKeys(); KeyIndex++)
		{
			for (int32 SubIndex = 0; SubIndex < DistUniformCurve->GetNumSubCurves(); SubIndex++)
			{
				float	Value	= DistUniformCurve->GetKeyOut(SubIndex, KeyIndex);
				DistUniformCurve->SetKeyOut(SubIndex, KeyIndex, Value * Multiplier);
			}
		}
	}
	else
	if (DistConstant)
	{
		DistConstant->Constant.X *= Multiplier;
		DistConstant->Constant.Y *= Multiplier;
		DistConstant->Constant.Z *= Multiplier;
	}
	else
	if (DistConstantCurve)
	{
		for (int32 KeyIndex = 0; KeyIndex < DistConstantCurve->GetNumKeys(); KeyIndex++)
		{
			for (int32 SubIndex = 0; SubIndex < DistConstantCurve->GetNumSubCurves(); SubIndex++)
			{
				float	Value	= DistConstantCurve->GetKeyOut(SubIndex, KeyIndex);
				DistConstantCurve->SetKeyOut(SubIndex, KeyIndex, Value * Multiplier);
			}
		}
	}
	else
	if (DistUniform)
	{
		DistUniform->Min.X	*= Multiplier;
		DistUniform->Min.Y	*= Multiplier;
		DistUniform->Min.Z	*= Multiplier;
		DistUniform->Max.X	*= Multiplier;
		DistUniform->Max.Y	*= Multiplier;
		DistUniform->Max.Z	*= Multiplier;
	}
	else
	{
		UE_LOGF(LogParticles, Log, "UParticleModule::ConvertVectorDistribution> Invalid distribution?");
		return false;
	}

	// Safety catch to ensure that the distribution lookup tables get rebuilt...
	VectorDist->bIsDirty = true;
	return true;
}

UParticleModule* UParticleModule::GenerateLODModule(UParticleLODLevel* SourceLODLevel, UParticleLODLevel* DestLODLevel, float Percentage, 
	bool bGenerateModuleData, bool bForceModuleConstruction)
{
	if (WillGeneratedModuleBeIdentical(SourceLODLevel, DestLODLevel, Percentage) && !bForceModuleConstruction)
	{
		LODValidity |= (1 << DestLODLevel->Level);
		return this;
	}

	// Otherwise, construct a new object and set the values appropriately... if required.
	UParticleModule* NewModule = NULL;

	UObject* DupObject = StaticDuplicateObject(this, GetOuter());
	if (DupObject)
	{
		NewModule = CastChecked<UParticleModule>(DupObject);
		NewModule->LODValidity = (1 << DestLODLevel->Level);
		if (bGenerateModuleData)
		{
			if (NewModule->GenerateLODModuleValues(this, Percentage, DestLODLevel) == false)
			{
				FString NameDump;
				GetName(NameDump);
				UE_LOGF(LogParticles, Log, "ERROR - GenerateFromLODLevel - Failed to generate LOD module values for %ls!", *NameDump);
				NewModule = NULL;
			}
		}
	}
	
	return NewModule;
}


bool UParticleModule::IsUsedInLODLevel(int32 SourceLODIndex) const
{
	if ((SourceLODIndex >= 0) && (SourceLODIndex <= 7))
	{
		return ((LODValidity & (1 << SourceLODIndex)) != 0);
	}
	return false;
}


void UParticleModule::GetParticleSysParamsUtilized(TArray<FString>& ParticleSysParamList)
{
}
						   

void UParticleModule::GetParticleParametersUtilized(TArray<FString>& ParticleParameterList)
{
	for (TFieldIterator<FStructProperty> It(GetClass()); It; ++It)
	{
		// attempt to get a distribution from a random struct property
		UObject* Distribution = FRawDistribution::TryGetDistributionObjectFromRawDistributionProperty(*It, (uint8*)this);
		if (Distribution)
		{
			UDistributionFloatParticleParameter* FloatPP = Cast<UDistributionFloatParticleParameter>(Distribution);
			UDistributionVectorParticleParameter* VectorPP = Cast<UDistributionVectorParticleParameter>(Distribution);

			// only handle particle param types
			if (FloatPP)
			{
				ParticleParameterList.Add(
					FString::Printf(
						TEXT("float : %32s - MinIn %10.5f, MaxIn %10.5f, MinOut %10.5f, MaxOut %10.5f, Mode %10s, Constant %10.5f\n"),
						*(FloatPP->ParameterName.ToString()),
						FloatPP->MinInput,
						FloatPP->MaxInput,
						FloatPP->MinOutput,
						FloatPP->MaxOutput,
						(FloatPP->ParamMode == DPM_Normal) ? TEXT("Normal") :
							(FloatPP->ParamMode == DPM_Abs) ? TEXT("Absolute") :
								(FloatPP->ParamMode == DPM_Direct) ? TEXT("Direct") :
									TEXT("????"),
						FloatPP->Constant)
						);
			}
			else 
			if (VectorPP)
			{
				FString ParamString;

				ParamString = FString::Printf(TEXT("VECTOR: %32s - "), *(VectorPP->ParameterName.ToString()));
				ParamString += FString::Printf(TEXT("MinIn %10.5f,%10.5f,%10.5f, "), 
					VectorPP->MinInput.X, VectorPP->MinInput.Y, VectorPP->MinInput.Z);
				ParamString += FString::Printf(TEXT("MaxIn %10.5f,%10.5f,%10.5f, "),
					VectorPP->MaxInput.X, VectorPP->MaxInput.Y, VectorPP->MaxInput.Z);
				ParamString += FString::Printf(TEXT("MinOut %10.5f,%10.5f,%10.5f, "),
						VectorPP->MinOutput.X, VectorPP->MinOutput.Y, VectorPP->MinOutput.Z);
				ParamString += FString::Printf(TEXT("MaxOut %10.5f,%10.5f,%10.5f, "),
						VectorPP->MaxOutput.X, VectorPP->MaxOutput.Y, VectorPP->MaxOutput.Z);
				ParamString += FString::Printf(TEXT("Mode %10s,%10s,%10s, "),
						(VectorPP->ParamModes[0] == DPM_Normal) ? TEXT("Normal") :
							(VectorPP->ParamModes[0] == DPM_Abs) ? TEXT("Absolute") :
								(VectorPP->ParamModes[0] == DPM_Direct) ? TEXT("Direct") :
									TEXT("????"),
						(VectorPP->ParamModes[1] == DPM_Normal) ? TEXT("Normal") :
							(VectorPP->ParamModes[1] == DPM_Abs) ? TEXT("Absolute") :
								(VectorPP->ParamModes[1] == DPM_Direct) ? TEXT("Direct") :
									TEXT("????"),
						(VectorPP->ParamModes[2] == DPM_Normal) ? TEXT("Normal") :
							(VectorPP->ParamModes[2] == DPM_Abs) ? TEXT("Absolute") :
								(VectorPP->ParamModes[2] == DPM_Direct) ? TEXT("Direct") :
									TEXT("????"));
				ParamString += FString::Printf(TEXT("Constant %10.5f,%10.5f,%10.5f\n"),
						VectorPP->Constant.X, VectorPP->Constant.Y, VectorPP->Constant.Z);
				ParticleParameterList.Add(ParamString);
			}
		}
	}
}


uint32 UParticleModule::PrepRandomSeedInstancePayload(FParticleEmitterInstance* Owner, FParticleRandomSeedInstancePayload* InRandSeedPayload, const FParticleRandomSeedInfo& InRandSeedInfo)
{
    // These should never be null
	if (!ensure(Owner != nullptr))
	{
		return 0xffffffff;
	}

	if (InRandSeedPayload != nullptr)
	{
		new(InRandSeedPayload) FParticleRandomSeedInstancePayload();

		// See if the parameter is set on the instance...
		if (InRandSeedInfo.bGetSeedFromInstance == true)
		{
			float SeedValue;
			if (Owner->Component.GetFloatParameter(InRandSeedInfo.ParameterName, SeedValue) == true)
			{
				if (InRandSeedInfo.bInstanceSeedIsIndex == false)
				{
					InRandSeedPayload->RandomStream.Initialize(FMath::RoundToInt(SeedValue));
				}
				else
				{
					if (InRandSeedInfo.RandomSeeds.Num() > 0)
					{
						int32 Index = FMath::Min<int32>((InRandSeedInfo.RandomSeeds.Num() - 1), FMath::TruncToInt(SeedValue));
						InRandSeedPayload->RandomStream.Initialize(InRandSeedInfo.RandomSeeds[Index]);
						return 0;
					}
					else
					{
						return 0xffffffff;
					}
				}
				return 0;
			}
		}

		// Pick a seed to use and initialize it!!!!
		if (InRandSeedInfo.RandomSeeds.Num() > 0)
		{
			int32 Index = 0;

			if (InRandSeedInfo.bRandomlySelectSeedArray)
			{
				Index = Owner->Component.GetRandomStream().RandHelper(InRandSeedInfo.RandomSeeds.Num());
			}

			InRandSeedPayload->RandomStream.Initialize(InRandSeedInfo.RandomSeeds[Index]);
			return 0;
		}
		else if (FApp::bUseFixedSeed)
		{
			InRandSeedPayload->RandomStream.Initialize(GetFName());
			return 0;
		}
	}
	return 0xffffffff;
}


bool UParticleModule::SetRandomSeedEntry(int32 InIndex, int32 InRandomSeed)
{
	FParticleRandomSeedInfo* SeedInfo = GetRandomSeedInfo();
	if (SeedInfo != NULL)
	{
		if (SeedInfo->RandomSeeds.Num() <= InIndex)
		{
			SeedInfo->RandomSeeds.AddZeroed(InIndex - SeedInfo->RandomSeeds.Num() + 1);
		}

		SeedInfo->RandomSeeds[InIndex] = InRandomSeed;
		return true;
	}
	return false;
}

bool UParticleModule::IsUsedInGPUEmitter()const
{
	UParticleSystem* Sys = Cast<UParticleSystem>(GetOuter());

	if( Sys )
	{
		for( int32 EmitterIdx=0 ; EmitterIdx < Sys->Emitters.Num() ; ++EmitterIdx )
		{
			UParticleEmitter* Emitter = Sys->Emitters[EmitterIdx];
			if( Emitter && Emitter->LODLevels.Num() )
			{
				//Have to make sure this module is used in this emitter before checking it's type data.
				bool bUsedInThisEmitter = false;
				for( int32 LodIdx=0 ; LodIdx < Emitter->LODLevels.Num() && !bUsedInThisEmitter ; ++LodIdx )
				{
					UParticleLODLevel* LODLevel = Emitter->LODLevels[LodIdx];
					if( LODLevel )
					{
						if( LODLevel->RequiredModule == this )
						{
							bUsedInThisEmitter = true;
						}
						else
						{
							for( int32 ModuleIdx=0 ; ModuleIdx < LODLevel->Modules.Num() && !bUsedInThisEmitter ; ++ModuleIdx )
							{
								UParticleModule* Module = LODLevel->Modules[ModuleIdx];
								if( Module == this )
								{
									bUsedInThisEmitter = true;
								}
							}
						}
					}
				}

				//If this module is used in this emitter then check it's type data and return whether it's GPU or not
				if( bUsedInThisEmitter )
				{
					//Can just check the highest lod.
					UParticleLODLevel* LODLevel = Emitter->LODLevels[0];
					if( LODLevel )
					{
						UParticleModule* TypeDataModule = LODLevel->TypeDataModule;
						if( TypeDataModule && TypeDataModule->IsA(UParticleModuleTypeDataGpu::StaticClass()) )
						{
							return true;//Module is used in a GPU emitter.
						}
					}
				}
			}
		}
	}

	return false;
}

FRandomStream& UParticleModule::GetRandomStream(const FContext& Context)
{
	FParticleRandomSeedInstancePayload* Payload = Context.Owner.GetModuleRandomSeedInstanceData(this);
	FRandomStream& RandomStream = (Payload != nullptr) ? Payload->RandomStream : Context.Owner.Component.GetRandomStream();
	return RandomStream;
}

#if WITH_EDITOR
void UParticleModule::SetTransactionFlag()
{
	SetFlags( RF_Transactional );

	for( TFieldIterator<FProperty> It(GetClass()); It; ++It )
	{
		FProperty* Property = *It;

		if( FStructProperty* StructProp = CastField<FStructProperty>(Property) )
		{
			UObject* Distribution = FRawDistribution::TryGetDistributionObjectFromRawDistributionProperty( StructProp, reinterpret_cast<uint8*>(this) );
			if( Distribution )
			{
				Distribution->SetFlags( RF_Transactional );
			}
		}
		else if( FObjectPropertyBase* ObjectPropertyBase = CastField<FObjectPropertyBase>(Property) )
		{
			if( (ObjectPropertyBase->PropertyClass == UDistributionFloat::StaticClass() ||
				 ObjectPropertyBase->PropertyClass == UDistributionVector::StaticClass()) )
			{
				//We've deprecated all the UFloatDistribution* usage and in PostLoad we set them to nullptr. TODO: can we just remove this code instead of a null check?
				if(UObject* Distribution = ObjectPropertyBase->GetObjectPropertyValue( ObjectPropertyBase->ContainerPtrToValuePtr<void>(this) ))
				{
					Distribution->SetFlags( RF_Transactional );
				}
				else
				{
					ensure(ObjectPropertyBase->HasAllPropertyFlags(CPF_Deprecated));
				}
			}
		}
		else if( FArrayProperty* ArrayProp = CastField<FArrayProperty>(Property) )
		{
			if( FStructProperty* InnerStructProp = CastField<FStructProperty>(ArrayProp->Inner) )
			{
				FScriptArrayHelper ArrayHelper( ArrayProp, Property->ContainerPtrToValuePtr<void>(this) );
				for( int32 Idx = 0; Idx < ArrayHelper.Num(); ++Idx )
				{
					for( FProperty* ArrayProperty = InnerStructProp->Struct->PropertyLink; ArrayProperty; ArrayProperty = ArrayProperty->PropertyLinkNext )
					{
						if( FStructProperty* ArrayStructProp = CastField<FStructProperty>(ArrayProperty) )
						{
							UObject* Distribution = FRawDistribution::TryGetDistributionObjectFromRawDistributionProperty( ArrayStructProp, ArrayHelper.GetRawPtr( Idx ) );
							if( Distribution )
							{
								Distribution->SetFlags( RF_Transactional );
							}
						}
					}
				}
			}
		}
	}
}

/** Helper archive class to find all references, used by the cycle finder **/
class FArchiveFixDistributionRefs : public FArchiveUObject
{
public:
	/**
	 * Constructor
	 *
	 * @param	Src		the object to serialize which may contain a references
	 */
	FArchiveFixDistributionRefs(UObject* Src)
		: Outer(Src)
	{
		// use the optimized RefLink to skip over properties which don't contain object references
		ArIsObjectReferenceCollector = true;

		ArIgnoreArchetypeRef				= true;
		ArIgnoreOuterRef					= true;
		ArIgnoreClassRef					= true;
		ArIsModifyingWeakAndStrongReferences = true;

		Src->Serialize( *this );
	}
	virtual FString GetArchiveName() const { return TEXT("FArchiveFindDistributionRefs"); }

	/** The particle module we are fixing **/
	UObject*	Outer;

private:
	/** Serialize a reference **/
	FArchive& operator<<( class UObject*& Obj )
	{
		if (Obj && Obj->IsA<UDistribution>())
		{
			if (Obj->GetOuter() != Outer )
			{
				UE_LOGF(LogParticles, Verbose, "Bad Module Distribution %ls not in %ls (resave asset will fix this)", *GetFullNameSafe(Obj), *GetFullNameSafe(Outer));
				UObject *New = (UObject*)FindObjectWithOuter(Outer, Obj->GetClass(), Obj->GetFName());
				if (New)
				{
					UE_LOGF(LogParticles, Verbose, "      Replacing with Found %ls", *GetFullNameSafe(New));
				}
				else
				{
					New = NewObject<UObject>(Outer, Obj->GetClass(), Obj->GetFName(), RF_NoFlags, Obj);
					UE_LOGF(LogParticles, Verbose, "      Replacing with New %ls", *GetFullNameSafe(New));
				}
				Obj = New;
			}
			Obj->ConditionalPostLoad();
		}
		return *this;
	}
};


void UParticleModule::PostLoadSubobjects(FObjectInstancingGraph* OuterInstanceGraph)
{
	Super::PostLoadSubobjects(OuterInstanceGraph);

	FArchiveFixDistributionRefs ArFind(this);

}

void UParticleModule::GetDistributionsRestrictedOnGPU(TArray<FString>& OutRestrictedDistributions)
{
	OutRestrictedDistributions.Add(TEXT("DistributionFloatParticleParameter"));
	OutRestrictedDistributions.Add(TEXT("DistributionVectorParticleParameter"));
}

bool UParticleModule::IsDistributionAllowedOnGPU(const UDistribution* Distribution)
{
	return !Distribution || !( Distribution->IsA(UDistributionFloatParticleParameter::StaticClass()) || Distribution->IsA(UDistributionVectorParticleParameter::StaticClass()) );
}

FText UParticleModule::GetDistributionNotAllowedOnGPUText(const FString& ModuleName, const FString& PropertyName)
{
	static FText DistNotAllowedOnGPUFormat = NSLOCTEXT("ParticleModules", "DistNotAllowedOnGPUFormat", "Distribution {0} in {1} is using a distribution that is not supported on GPU emitters.");
	return FText::Format(DistNotAllowedOnGPUFormat, FText::FromString(PropertyName), FText::FromString(ModuleName));
}


#endif

/*-----------------------------------------------------------------------------
	UParticleModuleSourceMovement implementation.
-----------------------------------------------------------------------------*/
UParticleModuleSourceMovement::UParticleModuleSourceMovement(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bFinalUpdateModule = true;
}

void UParticleModuleSourceMovement::InitializeDefaults()
{
	if (!SourceMovementScale.IsCreated())
	{
		UDistributionVectorConstant* DistributionSourceMovementScale = NewObject<UDistributionVectorConstant>(this, TEXT("DistributionSourceMovementScale"));
		DistributionSourceMovementScale->Constant = FVector(1.0f, 1.0f, 1.0f);
		SourceMovementScale.Distribution = DistributionSourceMovementScale;
	}
}

void UParticleModuleSourceMovement::PostInitProperties()
{
	Super::PostInitProperties();
	if (!HasAnyFlags(RF_ClassDefaultObject | RF_NeedLoad))
	{
		InitializeDefaults();
	}
}

#if WITH_EDITOR
void UParticleModuleSourceMovement::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	InitializeDefaults();
	Super::PostEditChangeProperty(PropertyChangedEvent);
}
#endif // WITH_EDITOR

void UParticleModuleSourceMovement::FinalUpdate(const FUpdateContext& Context)
{
	Super::FinalUpdate(Context);
	// If enabled, update the source offset translation for all particles...
	{
		FVector FrameDiff = Context.GetTransform().GetLocation() - Context.Owner.Component.GetOldPosition();
		BEGIN_UPDATE_LOOP;
		{
			// Rough estimation of the particle being alive for more than a frame
			if (Particle.RelativeTime > (2.0f * DeltaTime * Particle.OneOverMaxLifetime))
			{
				FVector	Scale = SourceMovementScale.GetValue(Particle.RelativeTime, Context.GetDistributionData());
				Particle.Location += (Scale * FrameDiff);
			}
		}
		END_UPDATE_LOOP;
	}
}

bool UParticleModuleSourceMovement::CanTickInAnyThread()
{
	return SourceMovementScale.OkForParallel();
}

/*-----------------------------------------------------------------------------
	UParticleModuleOrientationBase implementation.
-----------------------------------------------------------------------------*/
UParticleModuleOrientationBase::UParticleModuleOrientationBase(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

/*-----------------------------------------------------------------------------
	UParticleModuleOrientationAxisLock implementation.
-----------------------------------------------------------------------------*/
UParticleModuleOrientationAxisLock::UParticleModuleOrientationAxisLock(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bSpawnModule = true;
	bUpdateModule = true;
}

//    uint8 LockAxisFlags;
//    FVector LockAxis;
void UParticleModuleOrientationAxisLock::Spawn(const FSpawnContext& Context)
{
}

void UParticleModuleOrientationAxisLock::Update(const FUpdateContext& Context)
{
}

#if WITH_EDITOR
void UParticleModuleOrientationAxisLock::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	UObject* OuterObj = GetOuter();
	check(OuterObj);
	UParticleLODLevel* LODLevel = Cast<UParticleLODLevel>(OuterObj);
	if (LODLevel)
	{
		// The outer is incorrect - warn the user and handle it
		UE_LOGF(LogParticles, Warning, "UParticleModuleOrientationAxisLock has an incorrect outer... run FixupEmitters on package %ls",
			*(OuterObj->GetOutermost()->GetPathName()));
		OuterObj = LODLevel->GetOuter();
		UParticleEmitter* Emitter = Cast<UParticleEmitter>(OuterObj);
		check(Emitter);
		OuterObj = Emitter->GetOuter();
	}
	UParticleSystem* PartSys = CastChecked<UParticleSystem>(OuterObj);

	FProperty* PropertyThatChanged = PropertyChangedEvent.Property;
	if (PropertyThatChanged)
	{
		if (PropertyThatChanged->GetFName() == FName("LockAxisFlags"))
		{
			PartSys->PostEditChangeProperty(PropertyChangedEvent);
		}
	}
	Super::PostEditChangeProperty(PropertyChangedEvent);
}
#endif // WITH_EDITOR

void UParticleModuleOrientationAxisLock::SetLockAxis(EParticleAxisLock eLockFlags)
{
	LockAxisFlags = eLockFlags;
}

/*-----------------------------------------------------------------------------
	UParticleModuleRequired implementation.
-----------------------------------------------------------------------------*/
UParticleModuleRequired::UParticleModuleRequired(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	ScreenAlignment = PSA_Square;
	bSpawnModule = true;
	bUpdateModule = true;
	EmitterDuration = 1.0f;
	EmitterDurationLow = 0.0f;
	bEmitterDurationUseRange = false;
	EmitterDelay = 0.0f;
	EmitterDelayLow = 0.0f;
	bEmitterDelayUseRange = false;
	EmitterLoops = 0;	
	SubImages_Horizontal = 1;
	SubImages_Vertical = 1;
	bUseMaxDrawCount = true;
	MaxDrawCount = 500;
	LODDuplicate = true;
	NormalsSphereCenter = FVector(0.0f, 0.0f, 100.0f);
	NormalsCylinderDirection = FVector(0.0f, 0.0f, 1.0f);
	bUseLegacyEmitterTime = true;
	bSupportLargeWorldCoordinates = true;
	UVFlippingMode = EParticleUVFlipMode::None;
	BoundingMode = BVC_EightVertices;
	AlphaThreshold = 0.1f;
}

void UParticleModuleRequired::InitializeDefaults()
{
	if (!SpawnRate.IsCreated())
	{
		SpawnRate.Distribution = NewObject<UDistributionFloatConstant>(this, TEXT("RequiredDistributionSpawnRate"));
	}
}

void UParticleModuleRequired::PostInitProperties()
{
	Super::PostInitProperties();

	if (!HasAnyFlags(RF_ClassDefaultObject))
	{
		BoundingGeometryBuffer = new FSubUVBoundingGeometryBuffer(&DerivedData.BoundingGeometry);
	}

	if (!HasAnyFlags(RF_ClassDefaultObject | RF_NeedLoad))
	{
		InitializeDefaults();		
	}
}

void UParticleModuleRequired::Serialize(FStructuredArchive::FRecord Record)
{
	FArchive& UnderlyingArchive = Record.GetUnderlyingArchive();

	Super::Serialize(Record);

	UnderlyingArchive.UsingCustomVersion(FRenderingObjectVersion::GUID);
	if (UnderlyingArchive.CustomVer(FRenderingObjectVersion::GUID) >= FRenderingObjectVersion::MovedParticleCutoutsToRequiredModule)
	{
		bool bCooked = UnderlyingArchive.IsCooking();

		// Save a bool indicating whether this is cooked data
		// This is needed when loading cooked data, to know to serialize differently
		Record << SA_VALUE(TEXT("bCooked"), bCooked);

		if (FPlatformProperties::RequiresCookedData() && !bCooked && UnderlyingArchive.IsLoading())
		{
			UE_LOGF(LogParticles, Fatal, "This platform requires cooked packages, and this SubUV animation does not contain cooked data %ls.", *GetName());
		}

		if (bCooked)
		{
			DerivedData.Serialize(Record.EnterField(TEXT("DerivedData")));
		}
	}
}

#if WITH_EDITOR
void UParticleModuleRequired::PreEditChange(FProperty* PropertyThatChanged)
{
	Super::PreEditChange(PropertyThatChanged);

	// Particle rendering is reading from this UObject's properties directly, wait until all queued commands are done
	FlushRenderingCommands();
}

void UParticleModuleRequired::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	InitializeDefaults();

	SubImages_Horizontal = FMath::Max(SubImages_Horizontal, 1);
	SubImages_Vertical = FMath::Max(SubImages_Vertical, 1);

	BeginReleaseResource(BoundingGeometryBuffer);

	// Wait until unregister commands are processed on the RT
	FlushRenderingCommands();

	FProperty* PropertyThatChanged = PropertyChangedEvent.Property;
	if (PropertyThatChanged)
	{
		if (PropertyThatChanged->GetFName() == FName("MaxDrawCount"))
		{
			if (MaxDrawCount >= 0)
			{
				bUseMaxDrawCount = true;
			}
			else
			{
				bUseMaxDrawCount = false;
			}
		}
		else if (AlphaThreshold > 0 && PropertyThatChanged->GetFName() == FName("Material"))
		{
			GetDefaultCutout();
		}
	}

	if (CutoutTexture)
	{
		CacheDerivedData();
		InitBoundingGeometryBuffer();
	}

	Super::PostEditChangeProperty(PropertyChangedEvent);
}

bool UParticleModuleRequired::IsValidForLODLevel(UParticleLODLevel* LODLevel, FString& OutErrorString)
{
	bool bValid = true;

	if (LODLevel && LODLevel->TypeDataModule)
	{
		if( LODLevel->TypeDataModule->IsA(UParticleModuleTypeDataGpu::StaticClass()))
		{
			if (InterpolationMethod == PSUVIM_Random || InterpolationMethod == PSUVIM_Random_Blend)
			{
				OutErrorString = NSLOCTEXT("UnrealEd", "RandomSubUVForGPUEmitter", "Random SubUV interpolation is not supported with GPU particles.").ToString();
				bValid = false;
			}
		}
		else if (UParticleModuleTypeDataMesh* MeshTypeData = Cast<UParticleModuleTypeDataMesh>(LODLevel->TypeDataModule))
		{
			if (NamedMaterialOverrides.Num() > 0 && MeshTypeData->bOverrideMaterial)
			{
				OutErrorString = NSLOCTEXT("UnrealEd", "NamedMaterialOverriedsAndOverrideMaterialError", "Cannot use Named Material Overrides when using OverrideMaterial in the mesh type data module.").ToString();
			}
		}
	}

	return bValid;
}
#endif // WITH_EDITOR

void UParticleModuleRequired::PostLoad()
{
	Super::PostLoad();

	SubImages_Horizontal = FMath::Max(SubImages_Horizontal, 1);
	SubImages_Vertical = FMath::Max(SubImages_Vertical, 1);

	if (!FPlatformProperties::RequiresCookedData())
	{
		if (CutoutTexture)
		{
			CutoutTexture->ConditionalPostLoad();
			CacheDerivedData();
		}	
	}

	InitBoundingGeometryBuffer();
}

void UParticleModuleRequired::BeginDestroy()
{
	Super::BeginDestroy();

	if (BoundingGeometryBuffer)
	{
		BeginReleaseResource(BoundingGeometryBuffer);
		ReleaseFence.BeginFence();
	}
}

bool UParticleModuleRequired::IsReadyForFinishDestroy()
{
	bool bReady = Super::IsReadyForFinishDestroy();

	return bReady && ReleaseFence.IsFenceComplete();
}

void UParticleModuleRequired::FinishDestroy()
{
	delete BoundingGeometryBuffer;

	Super::FinishDestroy();
}

void UParticleModuleRequired::SetToSensibleDefaults(UParticleEmitter* Owner)
{
	Super::SetToSensibleDefaults(Owner);
	bUseLegacyEmitterTime = false;
}

bool UParticleModuleRequired::GenerateLODModuleValues(UParticleModule* SourceModule, float Percentage, UParticleLODLevel* LODLevel)
{
	// Convert the module values
	UParticleModuleRequired*	RequiredSource	= Cast<UParticleModuleRequired>(SourceModule);
	if (!RequiredSource)
	{
		return false;
	}

	bool bResult	= true;

	Material = RequiredSource->Material;
	ScreenAlignment = RequiredSource->ScreenAlignment;

	//bUseLocalSpace
	//bKillOnDeactivate
	//bKillOnCompleted
	//EmitterDuration
	//EmitterLoops
	//SpawnRate
	//InterpolationMethod
	//SubImages_Horizontal
	//SubImages_Vertical
	//bScaleUV
	//RandomImageTime
	//RandomImageChanges
	//SubUVDataOffset
	//EmitterRenderMode
	//EmitterEditorColor

	return bResult;
}

void UParticleModuleRequired::CacheDerivedData()
{
#if WITH_EDITORONLY_DATA
	const FString KeyString = FSubUVDerivedData::GetDDCKeyString(CutoutTexture->Source.GetId(), SubImages_Horizontal, SubImages_Vertical, (int32)BoundingMode, AlphaThreshold, (int32)OpacitySourceMode);
	TArray<uint8> Data;

	COOK_STAT(auto Timer = SubUVAnimationCookStats::UsageStats.TimeSyncWork());
	if (GetDerivedDataCacheRef().GetSynchronous(*KeyString, Data, GetPathName()))
	{
		COOK_STAT(Timer.AddHit(Data.Num()));
		DerivedData.BoundingGeometry.Empty(Data.Num() / sizeof(FVector2f));
		DerivedData.BoundingGeometry.AddUninitialized(Data.Num() / sizeof(FVector2f));
		FPlatformMemory::Memcpy(DerivedData.BoundingGeometry.GetData(), Data.GetData(), Data.Num() * Data.GetTypeSize());
	}
	else
	{
		DerivedData.Build(CutoutTexture, SubImages_Horizontal, SubImages_Vertical, BoundingMode, AlphaThreshold, OpacitySourceMode);

		Data.Empty(DerivedData.BoundingGeometry.Num() * sizeof(FVector2f));
		Data.AddUninitialized(DerivedData.BoundingGeometry.Num() * sizeof(FVector2f));
		FPlatformMemory::Memcpy(Data.GetData(), DerivedData.BoundingGeometry.GetData(), DerivedData.BoundingGeometry.Num() * DerivedData.BoundingGeometry.GetTypeSize());
		GetDerivedDataCacheRef().Put(*KeyString, Data, GetPathName());
		COOK_STAT(Timer.AddMiss(Data.Num()));
	}
#endif
}

bool UParticleModuleRequired::ShouldUseVelocityForMotionBlur() const
{
	if ( bOverrideUseVelocityForMotionBlur )
	{
		return bUseVelocityForMotionBlur;
	}
	return CVarFxCascadeUseVelocityForMotionBlur.GetValueOnAnyThread();
}

void UParticleModuleRequired::InitBoundingGeometryBuffer()
{
	// The SRV is only needed for platforms that can render particles with instancing
	if (BoundingGeometryBuffer->Vertices->Num())
	{
		BeginInitResource(BoundingGeometryBuffer);
	}
}

void UParticleModuleRequired::GetDefaultCutout()
{
#if WITH_EDITOR
	if (Material && GetDefault<URendererSettings>()->bDefaultParticleCutouts)
	{
		// Try to find an opacity mask texture to default to, if not try to find an opacity texture
		TArray<UTexture*> OpacityMaskTextures;
		Material->GetTexturesInPropertyChain(MP_OpacityMask, OpacityMaskTextures, nullptr, nullptr);

		if (OpacityMaskTextures.Num())
		{
			CutoutTexture = (UTexture2D*)OpacityMaskTextures[0];
		}
		else
		{
			TArray<UTexture*> OpacityTextures;
			Material->GetTexturesInPropertyChain(MP_Opacity, OpacityTextures, nullptr, nullptr);

			if (OpacityTextures.Num())
			{
				CutoutTexture = (UTexture2D*)OpacityTextures[0];
			}
		}
	}
#endif
}

/*-----------------------------------------------------------------------------
	UParticleModuleRotationBase implementation.
-----------------------------------------------------------------------------*/
UParticleModuleRotationBase::UParticleModuleRotationBase(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

/*-----------------------------------------------------------------------------
	UParticleModuleMeshRotation implementation.
-----------------------------------------------------------------------------*/
UParticleModuleMeshRotation::UParticleModuleMeshRotation(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bSpawnModule = true;
	bInheritParent = false;
}

void UParticleModuleMeshRotation::InitializeDefaults()
{
	if (!StartRotation.IsCreated())
	{
		UDistributionVectorUniform* DistributionStartRotation = NewObject<UDistributionVectorUniform>(this, TEXT("DistributionStartRotation"));
		DistributionStartRotation->Min = FVector(0.0f, 0.0f, 0.0f);
		DistributionStartRotation->Max = FVector(1.0f, 1.0f, 1.0f);
		StartRotation.Distribution = DistributionStartRotation;
	}
}


void UParticleModuleMeshRotation::PostInitProperties()
{
	Super::PostInitProperties();
	if (!HasAnyFlags(RF_ClassDefaultObject | RF_NeedLoad))
	{
		InitializeDefaults();
	}
}

#if WITH_EDITOR
void UParticleModuleMeshRotation::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	InitializeDefaults();
	Super::PostEditChangeProperty(PropertyChangedEvent);
}
#endif // WITH_EDITOR

void UParticleModuleMeshRotation::Spawn(const FSpawnContext& Context)
{
	SpawnEx(Context, &GetRandomStream(Context));
}


void UParticleModuleMeshRotation::SpawnEx(const FSpawnContext& Context, struct FRandomStream* InRandomStream)
{
	FParticleEmitterInstance* Owner = &Context.Owner;

	SPAWN_INIT;
	{
		const int32 MeshRotationOffset = Owner->GetMeshRotationOffset();
		if (MeshRotationOffset)
		{
			FVector Rotation = StartRotation.GetValue(Owner->EmitterTime, Context.GetDistributionData(), 0, InRandomStream);
			if (bInheritParent)
			{
				FRotator	Rotator	= Owner->Component.GetComponentRotation();
				FVector		ParentAffectedRotation	= Rotator.Euler();
				Rotation.X	+= ParentAffectedRotation.X / 360.0f;
				Rotation.Y	+= ParentAffectedRotation.Y / 360.0f;
				Rotation.Z	+= ParentAffectedRotation.Z / 360.0f;
			}
			FMeshRotationPayloadData* PayloadData = (FMeshRotationPayloadData*)((uint8*)&Particle + MeshRotationOffset);
			//PayloadData->Rotation.X	+= Rotation.X * 360.0f;
			//PayloadData->Rotation.Y	+= Rotation.Y * 360.0f;
			//PayloadData->Rotation.Z	+= Rotation.Z * 360.0f;
			PayloadData->InitRotation = FVector3f(Rotation) * 360.0f;
			PayloadData->Rotation += PayloadData->InitRotation;
		}
	}
}

/*-----------------------------------------------------------------------------
	UParticleModuleMeshRotation_Seeded implementation.
-----------------------------------------------------------------------------*/
UParticleModuleMeshRotation_Seeded::UParticleModuleMeshRotation_Seeded(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bSpawnModule = true;
	bSupportsRandomSeed = true;
	bRequiresLoopingNotification = true;
}

void UParticleModuleMeshRotation_Seeded::EmitterLoopingNotify(FParticleEmitterInstance* Owner)
{
	if (RandomSeedInfo.bResetSeedOnEmitterLooping == true)
	{
		FParticleRandomSeedInstancePayload* Payload = Owner->GetModuleRandomSeedInstanceData(this);
		PrepRandomSeedInstancePayload(Owner, Payload, RandomSeedInfo);
	}
}

/*-----------------------------------------------------------------------------
	UParticleModuleRotationRateBase implementation.
-----------------------------------------------------------------------------*/
UParticleModuleRotationRateBase::UParticleModuleRotationRateBase(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

/*-----------------------------------------------------------------------------
	UParticleModuleMeshRotationRate implementation.
-----------------------------------------------------------------------------*/
UParticleModuleMeshRotationRate::UParticleModuleMeshRotationRate(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bSpawnModule = true;
	bUpdateModule = true;
}

void UParticleModuleMeshRotationRate::InitializeDefaults()
{
	if (!StartRotationRate.IsCreated())
	{
		UDistributionVectorUniform* DistributionStartRotationRate = NewObject<UDistributionVectorUniform>(this, TEXT("DistributionStartRotationRate"));
		DistributionStartRotationRate->Min = FVector(0.0f, 0.0f, 0.0f);
		DistributionStartRotationRate->Max = FVector(360.0f, 360.0f, 360.0f);
		StartRotationRate.Distribution = DistributionStartRotationRate;
	}
}

void UParticleModuleMeshRotationRate::PostInitProperties()
{
	Super::PostInitProperties();
	if (!HasAnyFlags(RF_ClassDefaultObject | RF_NeedLoad))
	{
		InitializeDefaults();
	}
}

#if WITH_EDITOR
void UParticleModuleMeshRotationRate::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	InitializeDefaults();
	Super::PostEditChangeProperty(PropertyChangedEvent);
}
#endif // WITH_EDITOR

void UParticleModuleMeshRotationRate::Spawn(const FSpawnContext& Context)
{
	SpawnEx(Context, &GetRandomStream(Context));
}

void UParticleModuleMeshRotationRate::SpawnEx(const FSpawnContext& Context, struct FRandomStream* InRandomStream)
{
	FParticleEmitterInstance* Owner = &Context.Owner;

	SPAWN_INIT;
	{
		const int32 MeshRotationOffset = Owner->GetMeshRotationOffset();
		if (MeshRotationOffset)
		{
			FVector StartRate = StartRotationRate.GetValue(Owner->EmitterTime, Context.GetDistributionData(), 0, InRandomStream);// * ((float)PI/180.f);
			FVector3f StartValue;
			StartValue.X = StartRate.X * 360.0f;
			StartValue.Y = StartRate.Y * 360.0f;
			StartValue.Z = StartRate.Z * 360.0f;

			FMeshRotationPayloadData* PayloadData = (FMeshRotationPayloadData*)((uint8*)&Particle + MeshRotationOffset);
			PayloadData->RotationRateBase	+= StartValue;
			PayloadData->RotationRate		+= StartValue;
		}
	}
}

void UParticleModuleMeshRotationRate::SetToSensibleDefaults(UParticleEmitter* Owner)
{
	UDistributionVectorUniform* StartRotationRateDist = Cast<UDistributionVectorUniform>(StartRotationRate.Distribution);
	if (StartRotationRateDist)
	{
		StartRotationRateDist->Min = FVector::ZeroVector;
		StartRotationRateDist->Max = FVector(1.0f,1.0f,1.0f);
		StartRotationRateDist->bIsDirty = true;
	}
}

/*-----------------------------------------------------------------------------
	UParticleModuleMeshRotationRate_Seeded implementation.
-----------------------------------------------------------------------------*/
UParticleModuleMeshRotationRate_Seeded::UParticleModuleMeshRotationRate_Seeded(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bSpawnModule = true;
	bSupportsRandomSeed = true;
	bRequiresLoopingNotification = true;
}

void UParticleModuleMeshRotationRate_Seeded::EmitterLoopingNotify(FParticleEmitterInstance* Owner)
{
	if (RandomSeedInfo.bResetSeedOnEmitterLooping == true)
	{
		FParticleRandomSeedInstancePayload* Payload = Owner->GetModuleRandomSeedInstanceData(this);
		PrepRandomSeedInstancePayload(Owner, Payload, RandomSeedInfo);
	}
}

/*-----------------------------------------------------------------------------
	UParticleModuleMeshRotationRateMultiplyLife implementation.
-----------------------------------------------------------------------------*/
UParticleModuleMeshRotationRateMultiplyLife::UParticleModuleMeshRotationRateMultiplyLife(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bSpawnModule = true;
	bUpdateModule = true;
}
void UParticleModuleMeshRotationRateMultiplyLife::InitializeDefaults()
{
	if (!LifeMultiplier.IsCreated())
	{
		LifeMultiplier.Distribution = NewObject<UDistributionVectorConstant>(this, TEXT("DistributionLifeMultiplier"));
	}
}

void UParticleModuleMeshRotationRateMultiplyLife::PostInitProperties()
{
	Super::PostInitProperties();
	if (!HasAnyFlags(RF_ClassDefaultObject | RF_NeedLoad))
	{
		InitializeDefaults();
	}
}

#if WITH_EDITOR
void UParticleModuleMeshRotationRateMultiplyLife::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	InitializeDefaults();
	Super::PostEditChangeProperty(PropertyChangedEvent);
}
#endif // WITH_EDITOR


void UParticleModuleMeshRotationRateMultiplyLife::Spawn(const FSpawnContext& Context)
{
	FParticleEmitterInstance* Owner = &Context.Owner;

	const int32 MeshRotationOffset = Owner->GetMeshRotationOffset();
	if (MeshRotationOffset)
	{
		SPAWN_INIT;
		{
			FMeshRotationPayloadData* PayloadData = (FMeshRotationPayloadData*)((uint8*)&Particle + MeshRotationOffset);
			FVector3f RateScale(LifeMultiplier.GetValue(Particle.RelativeTime, Context.GetDistributionData()));
			PayloadData->RotationRate *= RateScale;
		}
	}
}

void UParticleModuleMeshRotationRateMultiplyLife::Update(const FUpdateContext& Context)
{
	FParticleEmitterInstance* Owner = &Context.Owner;
	const int32 MeshRotationOffset = Owner->GetMeshRotationOffset();
	if (MeshRotationOffset)
	{
		BEGIN_UPDATE_LOOP;
		{
			FMeshRotationPayloadData* PayloadData = (FMeshRotationPayloadData*)((uint8*)&Particle + MeshRotationOffset);
			FVector3f RateScale(LifeMultiplier.GetValue(Particle.RelativeTime, Context.GetDistributionData()));
			PayloadData->RotationRate *= RateScale;
		}
		END_UPDATE_LOOP;
	}
}

void UParticleModuleMeshRotationRateMultiplyLife::SetToSensibleDefaults(UParticleEmitter* Owner)
{
	UDistributionVectorUniform* LifeMultiplierDist = Cast<UDistributionVectorUniform>(LifeMultiplier.Distribution);
	if (LifeMultiplierDist)
	{
		LifeMultiplierDist->Min = FVector::ZeroVector;
		LifeMultiplierDist->Max = FVector(1.0f,1.0f,1.0f);
		LifeMultiplierDist->bIsDirty = true;
	}
}

/*-----------------------------------------------------------------------------
	UParticleModuleMeshRotationRateOverLife implementation.
-----------------------------------------------------------------------------*/
UParticleModuleMeshRotationRateOverLife::UParticleModuleMeshRotationRateOverLife(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bSpawnModule = true;
	bUpdateModule = true;
}

void UParticleModuleMeshRotationRateOverLife::InitializeDefaults()
{
	if (!RotRate.IsCreated())
	{
		RotRate.Distribution = NewObject<UDistributionVectorConstantCurve>(this, TEXT("DistributionRotRate"));
	}
}

void UParticleModuleMeshRotationRateOverLife::PostInitProperties()
{
	Super::PostInitProperties();
	if (!HasAnyFlags(RF_ClassDefaultObject | RF_NeedLoad))
	{
		InitializeDefaults();
	}
}

#if WITH_EDITOR
void UParticleModuleMeshRotationRateOverLife::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	InitializeDefaults();
	Super::PostEditChangeProperty(PropertyChangedEvent);
}
#endif // WITH_EDITOR

void UParticleModuleMeshRotationRateOverLife::Spawn(const FSpawnContext& Context)
{
	FParticleEmitterInstance* Owner = &Context.Owner;
	const int32 MeshRotationOffset = Owner->GetMeshRotationOffset();
	if (MeshRotationOffset)
	{
		SPAWN_INIT;
		{
			FMeshRotationPayloadData* PayloadData = (FMeshRotationPayloadData*)((uint8*)&Particle + MeshRotationOffset);
			FVector3f RateValue(RotRate.GetValue(Particle.RelativeTime, Context.GetDistributionData()));// * ((float)PI/180.f);
			RateValue.X = RateValue.X * 360.0f;
			RateValue.Y = RateValue.Y * 360.0f;
			RateValue.Z = RateValue.Z * 360.0f;

			if (bScaleRotRate == false)
			{
				PayloadData->RotationRate += RateValue;
			}
			else
			{
				PayloadData->RotationRate *= RateValue;
			}
		}
	}
}

void UParticleModuleMeshRotationRateOverLife::Update(const FUpdateContext& Context)
{
	FParticleEmitterInstance* Owner = &Context.Owner;
	const int32 MeshRotationOffset = Owner->GetMeshRotationOffset();
	if (MeshRotationOffset)
	{
		FMeshRotationPayloadData* PayloadData;
		FVector RateValue;
		if (bScaleRotRate == false)
		{
			BEGIN_UPDATE_LOOP;
			{
				PayloadData = (FMeshRotationPayloadData*)((uint8*)&Particle + MeshRotationOffset);
				RateValue = RotRate.GetValue(Particle.RelativeTime, Context.GetDistributionData());// * ((float)PI/180.f);
				RateValue.X = RateValue.X * 360.0f;
				RateValue.Y = RateValue.Y * 360.0f;
				RateValue.Z = RateValue.Z * 360.0f;
				PayloadData->RotationRate += FVector3f(RateValue);
			}
			END_UPDATE_LOOP;
		}
		else
		{
			BEGIN_UPDATE_LOOP;
			{
				PayloadData = (FMeshRotationPayloadData*)((uint8*)&Particle + MeshRotationOffset);
				RateValue = RotRate.GetValue(Particle.RelativeTime, Context.GetDistributionData());// * ((float)PI/180.f);
				RateValue.X = RateValue.X * 360.0f;
				RateValue.Y = RateValue.Y * 360.0f;
				RateValue.Z = RateValue.Z * 360.0f;
				PayloadData->RotationRate *= FVector3f(RateValue);
			}
			END_UPDATE_LOOP;
		}
	}
}

void UParticleModuleMeshRotationRateOverLife::SetToSensibleDefaults(UParticleEmitter* Owner)
{
	UDistributionVectorConstantCurve* RotRateDist = Cast<UDistributionVectorConstantCurve>(RotRate.Distribution);
	if (RotRateDist)
	{
		RotRateDist->ConstantCurve.AddPoint(0.0f, FVector::ZeroVector);
		RotRateDist->ConstantCurve.AddPoint(1.0f, FVector(1.0f, 1.0f, 1.0f));
		RotRateDist->bIsDirty = true;
	}
}

/*-----------------------------------------------------------------------------
	UParticleModuleRotation implementation.
-----------------------------------------------------------------------------*/
UParticleModuleRotation::UParticleModuleRotation(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bSpawnModule = true;
}

void UParticleModuleRotation::InitializeDefaults()
{
	if (!StartRotation.IsCreated())
	{
		UDistributionFloatUniform* DistributionStartRotation = NewObject<UDistributionFloatUniform>(this, TEXT("DistributionStartRotation"));
		DistributionStartRotation->Min = 0.0f;
		DistributionStartRotation->Max = 1.0f;
		StartRotation.Distribution = DistributionStartRotation;
	}
}

void UParticleModuleRotation::PostInitProperties()
{
	Super::PostInitProperties();
	if (!HasAnyFlags(RF_ClassDefaultObject | RF_NeedLoad))
	{
		InitializeDefaults();
	}
}

#if WITH_EDITOR
void UParticleModuleRotation::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	InitializeDefaults();
	Super::PostEditChangeProperty(PropertyChangedEvent);
}
#endif // WITH_EDITOR

void UParticleModuleRotation::Spawn(const FSpawnContext& Context)
{
	SpawnEx(Context, &GetRandomStream(Context));
}

void UParticleModuleRotation::SpawnEx(const FSpawnContext& Context, struct FRandomStream* InRandomStream)
{
	FParticleEmitterInstance* Owner = &Context.Owner;

	SPAWN_INIT;
	{
		Particle.Rotation += (UE_PI/180.f) * 360.0f * StartRotation.GetValue(Owner->EmitterTime, Context.GetDistributionData(), InRandomStream);
	}
}

/*-----------------------------------------------------------------------------
	UParticleModuleRotation_Seeded implementation.
-----------------------------------------------------------------------------*/
UParticleModuleRotation_Seeded::UParticleModuleRotation_Seeded(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bSpawnModule = true;
	bSupportsRandomSeed = true;
	bRequiresLoopingNotification = true;
}

void UParticleModuleRotation_Seeded::EmitterLoopingNotify(FParticleEmitterInstance* Owner)
{
	if (RandomSeedInfo.bResetSeedOnEmitterLooping == true)
	{
		FParticleRandomSeedInstancePayload* Payload = Owner->GetModuleRandomSeedInstanceData(this);
		PrepRandomSeedInstancePayload(Owner, Payload, RandomSeedInfo);
	}
}

/*-----------------------------------------------------------------------------
	UParticleModuleRotationRate implementation.
-----------------------------------------------------------------------------*/
UParticleModuleRotationRate::UParticleModuleRotationRate(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bSpawnModule = true;
}

void UParticleModuleRotationRate::InitializeDefaults()
{
	if (!StartRotationRate.IsCreated())
	{
		StartRotationRate.Distribution = NewObject<UDistributionFloatConstant>(this, TEXT("DistributionStartRotationRate"));
	}
}

void UParticleModuleRotationRate::PostInitProperties()
{
	Super::PostInitProperties();
	if (!HasAnyFlags(RF_ClassDefaultObject | RF_NeedLoad))
	{
		InitializeDefaults();
	}
}

void UParticleModuleRotationRate::CompileModule( FParticleEmitterBuildInfo& EmitterInfo )
{
	float MinRate;
	float MaxRate;

	// Call GetValue once to ensure the distribution has been initialized.
	StartRotationRate.GetValue();
	StartRotationRate.GetOutRange( MinRate, MaxRate );
	EmitterInfo.MaxRotationRate = MaxRate;
	EmitterInfo.SpawnModules.Add(this);
}

#if WITH_EDITOR
void UParticleModuleRotationRate::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	InitializeDefaults();
	Super::PostEditChangeProperty(PropertyChangedEvent);
}
#endif // WITH_EDITOR

void UParticleModuleRotationRate::Spawn(const FSpawnContext& Context)
{
	SpawnEx(Context, &GetRandomStream(Context));
}

void UParticleModuleRotationRate::SpawnEx(const FSpawnContext& Context, struct FRandomStream* InRandomStream)
{
	FParticleEmitterInstance* Owner = &Context.Owner;

	SPAWN_INIT;
	{
		float StartRotRate = (UE_PI/180.f) * 360.0f * StartRotationRate.GetValue(Owner->EmitterTime, Context.GetDistributionData(), InRandomStream);
		Particle.RotationRate += StartRotRate;
		Particle.BaseRotationRate += StartRotRate;
	}
}

void UParticleModuleRotationRate::SetToSensibleDefaults(UParticleEmitter* Owner)
{
	StartRotationRate.Distribution = NewObject<UDistributionFloatUniform>(this);
	UDistributionFloatUniform* StartRotationRateDist = Cast<UDistributionFloatUniform>(StartRotationRate.Distribution);
	if (StartRotationRateDist)
	{
		StartRotationRateDist->Min = 0.0f;
		StartRotationRateDist->Max = 1.0f;
		StartRotationRateDist->bIsDirty = true;
	}
}

/*-----------------------------------------------------------------------------
	UParticleModuleRotationRate_Seeded implementation.
-----------------------------------------------------------------------------*/
UParticleModuleRotationRate_Seeded::UParticleModuleRotationRate_Seeded(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bSpawnModule = true;
	bSupportsRandomSeed = true;
	bRequiresLoopingNotification = true;
}

void UParticleModuleRotationRate_Seeded::EmitterLoopingNotify(FParticleEmitterInstance* Owner)
{
	if (RandomSeedInfo.bResetSeedOnEmitterLooping == true)
	{
		FParticleRandomSeedInstancePayload* Payload = Owner->GetModuleRandomSeedInstanceData(this);
		PrepRandomSeedInstancePayload(Owner, Payload, RandomSeedInfo);
	}
}

/*-----------------------------------------------------------------------------
	UParticleModuleRotationOverLifetime implementation.
-----------------------------------------------------------------------------*/
UParticleModuleRotationOverLifetime::UParticleModuleRotationOverLifetime(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bSpawnModule = false;
	bUpdateModule = true;	
	Scale = true;
}

void UParticleModuleRotationOverLifetime::InitializeDefaults()
{
	if (!RotationOverLife.IsCreated())
	{
		RotationOverLife.Distribution = NewObject<UDistributionFloatConstantCurve>(this, TEXT("DistributionRotOverLife"));
	}
}

void UParticleModuleRotationOverLifetime::PostInitProperties()
{
	Super::PostInitProperties();
	if (!HasAnyFlags(RF_ClassDefaultObject | RF_NeedLoad))
	{
		RotationOverLife.Distribution = NewObject<UDistributionFloatConstantCurve>(this, TEXT("DistributionRotOverLife"));
	}
}

#if WITH_EDITOR
void UParticleModuleRotationOverLifetime::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	InitializeDefaults();
	Super::PostEditChangeProperty(PropertyChangedEvent);
}
#endif // WITH_EDITOR

void UParticleModuleRotationOverLifetime::Update(const FUpdateContext& Context)
{
	FParticleEmitterInstance* Owner = &Context.Owner;
	if (Scale)
	{
		BEGIN_UPDATE_LOOP;
		{
			float Rotation = RotationOverLife.GetValue(Particle.RelativeTime, Context.GetDistributionData());
			// For now, we are just using the X-value
			Particle.Rotation = (Particle.Rotation * (Rotation * (UE_PI/180.f) * 360.0f));
		}
		END_UPDATE_LOOP;
	}
	else
	{
		BEGIN_UPDATE_LOOP;
		{
			float Rotation = RotationOverLife.GetValue(Particle.RelativeTime, Context.GetDistributionData());
			// For now, we are just using the X-value
			Particle.Rotation = (Particle.Rotation + (Rotation * (UE_PI/180.f) * 360.0f));
		}
		END_UPDATE_LOOP;
	}
}

/*-----------------------------------------------------------------------------
	UParticleModuleSubUVBase implementation.
-----------------------------------------------------------------------------*/
UParticleModuleSubUVBase::UParticleModuleSubUVBase(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

/*-----------------------------------------------------------------------------
	UParticleModuleSubUV implementation.
-----------------------------------------------------------------------------*/
UParticleModuleSubUV::UParticleModuleSubUV(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bSpawnModule = true;
	bUpdateModule = true;	
}

void UParticleModuleSubUV::InitializeDefaults()
{
	if (!SubImageIndex.IsCreated())
	{
		SubImageIndex.Distribution = NewObject<UDistributionFloatConstant>(this, TEXT("DistributionSubImage"));
	}
}

void UParticleModuleSubUV::PostLoad()
{
	Super::PostLoad();

	if (Animation)
	{
		Animation->ConditionalPostLoad();
	}	
}

bool UParticleModuleSubUV::IsPostLoadThreadSafe() const
{
	return false;
}

void UParticleModuleSubUV::PostInitProperties()
{
	Super::PostInitProperties();
	if (!HasAnyFlags(RF_ClassDefaultObject | RF_NeedLoad))
	{
		SubImageIndex.Distribution = NewObject<UDistributionFloatConstant>(this, TEXT("DistributionSubImage"));
	}
}

void UParticleModuleSubUV::CompileModule( FParticleEmitterBuildInfo& EmitterInfo )
{
	check( EmitterInfo.RequiredModule );
	EParticleSubUVInterpMethod InterpMethod = (EParticleSubUVInterpMethod)EmitterInfo.RequiredModule->InterpolationMethod;
	if ( InterpMethod == PSUVIM_Linear || InterpMethod == PSUVIM_Linear_Blend )
	{
		EmitterInfo.SubImageIndex.Initialize( SubImageIndex.Distribution );
	}
}

#if WITH_EDITOR
void UParticleModuleSubUV::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	InitializeDefaults();
	Super::PostEditChangeProperty(PropertyChangedEvent);
}

bool UParticleModuleSubUV::IsValidForLODLevel(UParticleLODLevel* LODLevel, FString& OutErrorString)
{
	if (LODLevel->TypeDataModule && LODLevel->TypeDataModule->IsA(UParticleModuleTypeDataGpu::StaticClass()))
	{
		if(!IsDistributionAllowedOnGPU(SubImageIndex.Distribution))
		{
			OutErrorString = GetDistributionNotAllowedOnGPUText(StaticClass()->GetName(), "SubImageIndex" ).ToString();
			return false;
		}
	}

	if (Animation && Animation->SubUVTexture)
	{
		bool bFoundTexture = false;

		if (LODLevel->RequiredModule->Material)
		{
			TArray<UTexture*> UsedTextures;
			LODLevel->RequiredModule->Material->GetUsedTextures(UsedTextures, TOptional<EMaterialQualityLevel::Type>(), GetMaxShaderPlatformChecked());

			for (int32 TextureIndex = 0; TextureIndex < UsedTextures.Num(); TextureIndex++)
			{
				if (UsedTextures[TextureIndex] == Animation->SubUVTexture)
				{
					bFoundTexture = true;
					break;
				}
			}
		}		

		if (!bFoundTexture)
		{
			OutErrorString = NSLOCTEXT("UnrealEd", "SubUVAnimationMismatch", "SubUV module has an Animation set whose texture doesn't match what the material is using.  Particles may not appear or have visible clipping.").ToString();
			return false;
		}
	}

	return true;
}
#endif // WITH_EDITOR

void UParticleModuleSubUV::Spawn(const FSpawnContext& Context)
{
	FParticleEmitterInstance* Owner = &Context.Owner;

	check(Owner->SpriteTemplate);

	UParticleLODLevel* LODLevel	= Owner->SpriteTemplate->GetCurrentLODLevel(Owner);
	check(LODLevel);
	// Grab the interpolation method...
	EParticleSubUVInterpMethod InterpMethod = (EParticleSubUVInterpMethod)(LODLevel->RequiredModule->InterpolationMethod);
	const int32 PayloadOffset = Owner->SubUVDataOffset;
	if ((InterpMethod == PSUVIM_None) || (PayloadOffset == 0))
	{
		return;
	}

	if (!LODLevel->TypeDataModule || LODLevel->TypeDataModule->SupportsSubUV())
	{
		SPAWN_INIT;
		{
			int32	TempOffset	= CurrentOffset;
			CurrentOffset	= PayloadOffset;
			PARTICLE_ELEMENT(FFullSubUVPayload, SubUVPayload);
			CurrentOffset	= TempOffset;

			SubUVPayload.ImageIndex = DetermineImageIndex(Context, Context.Offset, &Particle, InterpMethod, SubUVPayload, Context.SpawnTime);
		}
	}
}

void UParticleModuleSubUV::Update(const FUpdateContext& Context)
{
	FParticleEmitterInstance* Owner = &Context.Owner;
	check(Owner->SpriteTemplate);

	UParticleLODLevel* LODLevel	= Owner->SpriteTemplate->GetCurrentLODLevel(Owner);
	// Grab the interpolation method...
	EParticleSubUVInterpMethod InterpMethod = 
		(EParticleSubUVInterpMethod)(LODLevel->RequiredModule->InterpolationMethod);
	const int32 PayloadOffset = Owner->SubUVDataOffset;
	if ((InterpMethod == PSUVIM_None) || (PayloadOffset == 0))
	{
		return;
	}

	// Quick-out in case of Random that only uses a single image for the whole lifetime...
	if ((InterpMethod == PSUVIM_Random) || (InterpMethod == PSUVIM_Random_Blend))
	{
		if (LODLevel->RequiredModule->RandomImageChanges == 0)
		{
			// Never change the random image...
			return;
		}
	}

	if (!LODLevel->TypeDataModule || LODLevel->TypeDataModule->SupportsSubUV())
	{
		BEGIN_UPDATE_LOOP;
			if (Particle.RelativeTime > 1.0f)
			{
				CONTINUE_UPDATE_LOOP;
			}

			int32	TempOffset	= CurrentOffset;
			CurrentOffset	= PayloadOffset;
			PARTICLE_ELEMENT(FFullSubUVPayload, SubUVPayload);
			CurrentOffset	= TempOffset;

			SubUVPayload.ImageIndex = DetermineImageIndex(Context, Offset, &Particle, InterpMethod, SubUVPayload, DeltaTime);
		END_UPDATE_LOOP;
	}
}

float UParticleModuleSubUV::DetermineImageIndex(const FContext& Context, int32 Offset, FBaseParticle* Particle, 
	EParticleSubUVInterpMethod InterpMethod, FFullSubUVPayload& SubUVPayload, float DeltaTime)
{
	FParticleEmitterInstance* Owner = &Context.Owner;

	UParticleLODLevel* LODLevel	= Owner->SpriteTemplate->GetCurrentLODLevel(Owner);
	check(LODLevel);

	USubUVAnimation* RESTRICT SubUVAnimation = Owner->SpriteTemplate->SubUVAnimation;

	const int32 TotalSubImages = SubUVAnimation 
		? SubUVAnimation->SubImages_Horizontal * SubUVAnimation->SubImages_Vertical
		: LODLevel->RequiredModule->SubImages_Horizontal * LODLevel->RequiredModule->SubImages_Vertical;

	float ImageIndex = SubUVPayload.ImageIndex;

	if ((InterpMethod == PSUVIM_Linear) || (InterpMethod == PSUVIM_Linear_Blend))
	{
		if (bUseRealTime == false)
		{
			ImageIndex = SubImageIndex.GetValue(Particle->RelativeTime, Context.GetDistributionData());
		}
		else
		{
			ImageIndex = SubImageIndex.GetValue(Particle->RelativeTime / Owner->Component.GetWorldEffectiveTimeDilation(), Context.GetDistributionData());
		}

		if (InterpMethod == PSUVIM_Linear)
		{
			ImageIndex = FMath::TruncToFloat(ImageIndex);
		}
	}
	else if ((InterpMethod == PSUVIM_Random) || (InterpMethod == PSUVIM_Random_Blend))
	{
		if ((LODLevel->RequiredModule->RandomImageTime == 0.0f) ||
			((Particle->RelativeTime - SubUVPayload.RandomImageTime) > LODLevel->RequiredModule->RandomImageTime) ||
			(SubUVPayload.RandomImageTime == 0.0f))
		{
			ImageIndex = GetRandomStream(Context).RandHelper(TotalSubImages);
			SubUVPayload.RandomImageTime	= Particle->RelativeTime;
		}

		if (InterpMethod == PSUVIM_Random)
		{
			ImageIndex = FMath::TruncToFloat(ImageIndex);
		}
	}
	else
	{
		ImageIndex	= 0;
	}

	return ImageIndex;
}

void UParticleModuleSubUV::SetToSensibleDefaults(UParticleEmitter* Owner)
{
	SubImageIndex.Distribution = NewObject<UDistributionFloatConstantCurve>(this);
	UDistributionFloatConstantCurve* SubImageIndexDist = Cast<UDistributionFloatConstantCurve>(SubImageIndex.Distribution);
	if (SubImageIndexDist)
	{
		// Add two points, one at time 0.0f and one at 1.0f
		for (int32 Key = 0; Key < 2; Key++)
		{
			int32	KeyIndex = SubImageIndexDist->CreateNewKey(Key * 1.0f);
			SubImageIndexDist->SetKeyOut(0, KeyIndex, 0.0f);
		}
		SubImageIndexDist->bIsDirty = true;
	}
}

/*-----------------------------------------------------------------------------
	UParticleModuleSubUVMovie implementation.
-----------------------------------------------------------------------------*/
UParticleModuleSubUVMovie::UParticleModuleSubUVMovie(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bSpawnModule = true;
	bUpdateModule = true;

	StartingFrame = 1;
}

void UParticleModuleSubUVMovie::InitializeDefaults()
{
	if (!FrameRate.IsCreated())
	{
		UDistributionFloatConstant* DistributionFrameRate = NewObject<UDistributionFloatConstant>(this, TEXT("DistributionFrameRate"));
		DistributionFrameRate->Constant = 30.0f;
		FrameRate.Distribution = DistributionFrameRate;
	}
}

void UParticleModuleSubUVMovie::PostInitProperties()
{
	Super::PostInitProperties();
	if (!HasAnyFlags(RF_ClassDefaultObject | RF_NeedLoad))
	{
		InitializeDefaults();
	}
}

#if WITH_EDITOR
void UParticleModuleSubUVMovie::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	InitializeDefaults();
	Super::PostEditChangeProperty(PropertyChangedEvent);
}
#endif // WITH_EDITOR

/**
 *	Helper structure for the particle payload of the SubUVMovie module.
 */
struct FSubUVMovieParticlePayload
{
	/** The time the particle has been alive, in realtime (seconds) */
	float	Time;
};


void UParticleModuleSubUVMovie::Spawn(const FSpawnContext& Context)
{
	FParticleEmitterInstance* Owner = &Context.Owner;
	check(Owner->SpriteTemplate);

	UParticleLODLevel* LODLevel	= Owner->SpriteTemplate->GetCurrentLODLevel(Owner);
	check(LODLevel);
	// Grab the interpolation method...
	int32	SubUVDataOffset = Owner->SubUVDataOffset;
	EParticleSubUVInterpMethod InterpMethod = (EParticleSubUVInterpMethod)(LODLevel->RequiredModule->InterpolationMethod);
	if ((InterpMethod == PSUVIM_None) || (SubUVDataOffset == 0))
	{
		return;
	}

	// Movies only work w/ Linear modes...
	if ((InterpMethod != PSUVIM_Linear) && (InterpMethod != PSUVIM_Linear_Blend))
	{
		return UParticleModuleSubUV::Spawn(Context);
	}

	if (!LODLevel->TypeDataModule || LODLevel->TypeDataModule->SupportsSubUV())
	{
		USubUVAnimation* RESTRICT SubUVAnimation = Owner->SpriteTemplate->SubUVAnimation;

		int32 TotalSubImages = SubUVAnimation 
			? SubUVAnimation->SubImages_Horizontal * SubUVAnimation->SubImages_Vertical
			: LODLevel->RequiredModule->SubImages_Horizontal * LODLevel->RequiredModule->SubImages_Vertical;

		if (TotalSubImages == 0)
		{
			TotalSubImages = 1;
		}

		SPAWN_INIT;
		{
			int32	TempOffset	= CurrentOffset;
			CurrentOffset	= SubUVDataOffset;
			PARTICLE_ELEMENT(FFullSubUVPayload, SubUVPayload);
			CurrentOffset	= TempOffset;

			const float UserSetFrameRate = FrameRate.GetValue(bUseEmitterTime ? Owner->EmitterTime : Particle.RelativeTime, Context.GetDistributionData());
			PARTICLE_ELEMENT(FSubUVMovieParticlePayload, MoviePayload);
			MoviePayload.Time = 0.0f;
			if (StartingFrame > 1)
			{
				// Clamp to the max...
				MoviePayload.Time = FMath::Clamp<float>(StartingFrame, 0, TotalSubImages-1);
			}
			else if (StartingFrame == 0)
			{
				FRandomStream& RandomStream = GetRandomStream(Context);
				MoviePayload.Time = FMath::TruncToFloat(RandomStream.FRand() * (TotalSubImages-1));
			}

			// Update the payload
			SubUVPayload.ImageIndex = MoviePayload.Time * UserSetFrameRate;
		}
	}
}


uint32 UParticleModuleSubUVMovie::RequiredBytes(UParticleModuleTypeDataBase* TypeData)
{
	return sizeof(FSubUVMovieParticlePayload);
}


float UParticleModuleSubUVMovie::DetermineImageIndex(const FContext& Context, int32 Offset, FBaseParticle* Particle, 
	EParticleSubUVInterpMethod InterpMethod, FFullSubUVPayload& SubUVPayload, float DeltaTime)
{
	FParticleEmitterInstance* Owner = &Context.Owner;
	FSubUVMovieParticlePayload& MoviePayload = *((FSubUVMovieParticlePayload*)((uint8*)Particle + Offset));
	float UserSetFrameRate = FrameRate.GetValue(bUseEmitterTime ? Owner->EmitterTime : Particle->RelativeTime, Context.GetDistributionData());
	if (bUseRealTime == false)
	{
		MoviePayload.Time += DeltaTime;
	}
	else
	{
		MoviePayload.Time += DeltaTime / Owner->Component.GetWorldEffectiveTimeDilation();
	}
	
	float ImageIndex = MoviePayload.Time * UserSetFrameRate;
	if (InterpMethod != PSUVIM_Linear_Blend)
	{
		ImageIndex = FMath::TruncToFloat(ImageIndex);
	}
	return ImageIndex;
}


void UParticleModuleSubUVMovie::SetToSensibleDefaults(UParticleEmitter* Owner)
{
	UParticleModuleSubUV::SetToSensibleDefaults(Owner);
}

void UParticleModuleSubUVMovie::GetCurveObjects(TArray<FParticleCurvePair>& OutCurves)
{
	UParticleModule::GetCurveObjects(OutCurves);

	int32 RemoveIdx = -1;
	for (int32 CurveIdx = 0; CurveIdx < OutCurves.Num(); CurveIdx++)
	{
		FParticleCurvePair& Curve = OutCurves[CurveIdx];
		if (Curve.CurveName == TEXT("SubImageIndex"))
		{
			RemoveIdx = CurveIdx;
			break;
		}
	}

	if (RemoveIdx != -1)
	{
		OutCurves.RemoveAt(RemoveIdx);
	}
}

/*-----------------------------------------------------------------------------
	UParticleModuleRotationRateMultiplyLife implementation.
-----------------------------------------------------------------------------*/
UParticleModuleRotationRateMultiplyLife::UParticleModuleRotationRateMultiplyLife(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bSpawnModule = true;
	bUpdateModule = true;
}

void UParticleModuleRotationRateMultiplyLife::InitializeDefaults()
{
	if (!LifeMultiplier.IsCreated())
	{
		LifeMultiplier.Distribution = NewObject<UDistributionFloatConstant>(this, TEXT("DistributionLifeMultiplier"));
	}
}

void UParticleModuleRotationRateMultiplyLife::PostInitProperties()
{
	Super::PostInitProperties();
	if (!HasAnyFlags(RF_ClassDefaultObject | RF_NeedLoad))
	{
		InitializeDefaults();
	}
}

#if WITH_EDITOR
void UParticleModuleRotationRateMultiplyLife::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	InitializeDefaults();
	Super::PostEditChangeProperty(PropertyChangedEvent);
}
#endif // WITH_EDITOR

void UParticleModuleRotationRateMultiplyLife::Spawn(const FSpawnContext& Context)
{
	FParticleEmitterInstance* Owner = &Context.Owner;
	SPAWN_INIT;
	float RateScale = LifeMultiplier.GetValue(Particle.RelativeTime, Context.GetDistributionData());
	Particle.RotationRate *= RateScale;
}

void UParticleModuleRotationRateMultiplyLife::Update(const FUpdateContext& Context)
{
	FParticleEmitterInstance* Owner = &Context.Owner;
	BEGIN_UPDATE_LOOP;
		float RateScale = LifeMultiplier.GetValue(Particle.RelativeTime, Context.GetDistributionData());
		Particle.RotationRate *= RateScale;
	END_UPDATE_LOOP;
}

void UParticleModuleRotationRateMultiplyLife::SetToSensibleDefaults(UParticleEmitter* Owner)
{
	LifeMultiplier.Distribution = NewObject<UDistributionFloatConstantCurve>(this);
	UDistributionFloatConstantCurve* LifeMultiplierDist = Cast<UDistributionFloatConstantCurve>(LifeMultiplier.Distribution);
	if (LifeMultiplierDist)
	{
		// Add two points, one at time 0.0f and one at 1.0f
		for (int32 Key = 0; Key < 2; Key++)
		{
			int32	KeyIndex = LifeMultiplierDist->CreateNewKey(Key * 1.0f);
			LifeMultiplierDist->SetKeyOut(0, KeyIndex, 1.0f);
		}
		LifeMultiplierDist->bIsDirty = true;
	}
}

/*-----------------------------------------------------------------------------
	UParticleModuleAcceleration implementation.
-----------------------------------------------------------------------------*/
UParticleModuleAccelerationBase::UParticleModuleAccelerationBase(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

void UParticleModuleAccelerationBase::SetToSensibleDefaults(UParticleEmitter* Owner)
{
	bAlwaysInWorldSpace = true;
	UParticleModule::SetToSensibleDefaults(Owner);
}

/*-----------------------------------------------------------------------------
	ParticleModuleAccelerationConstant implementation.
-----------------------------------------------------------------------------*/
UParticleModuleAccelerationConstant::UParticleModuleAccelerationConstant(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bSpawnModule = true;
	bUpdateModule = true;
}

void UParticleModuleAccelerationConstant::CompileModule( FParticleEmitterBuildInfo& EmitterInfo )
{
	EmitterInfo.ConstantAcceleration = Acceleration;
}

void UParticleModuleAccelerationConstant::Spawn(const FSpawnContext& Context)
{
	FParticleEmitterInstance* Owner = &Context.Owner;
	SPAWN_INIT;
	UParticleLODLevel* LODLevel	= Owner->SpriteTemplate->GetCurrentLODLevel(Owner);
	check(LODLevel);
	if (bAlwaysInWorldSpace && LODLevel->RequiredModule->bUseLocalSpace)
	{
		FVector3f LocalAcceleration(Context.GetTransform().InverseTransformVector(Acceleration));
		Particle.Velocity		+= LocalAcceleration * Context.SpawnTime;
		Particle.BaseVelocity	+= LocalAcceleration * Context.SpawnTime;
	}
	else
	{
		FVector3f LocalAcceleration(Acceleration);
		if (LODLevel->RequiredModule->bUseLocalSpace)
		{
			LocalAcceleration = FVector4f(Owner->EmitterToSimulation.TransformVector((FVector)LocalAcceleration));
		}
		Particle.Velocity		+= LocalAcceleration * Context.SpawnTime;
		Particle.BaseVelocity	+= LocalAcceleration * Context.SpawnTime;
	}
}

void UParticleModuleAccelerationConstant::Update(const FUpdateContext& Context)
{
	FParticleEmitterInstance* Owner = &Context.Owner;
	if ((Owner == NULL) || (Owner->ActiveParticles <= 0) || 
		(Owner->ParticleData == NULL) || (Owner->ParticleIndices == NULL))
	{
		return;
	}
	UParticleLODLevel* LODLevel	= Owner->SpriteTemplate->GetCurrentLODLevel(Owner);
	check(LODLevel);
	FPlatformMisc::Prefetch(Owner->ParticleData, (Owner->ParticleIndices[0] * Owner->ParticleStride));
	FPlatformMisc::Prefetch(Owner->ParticleData, (Owner->ParticleIndices[0] * Owner->ParticleStride) + PLATFORM_CACHE_LINE_SIZE);
	if (bAlwaysInWorldSpace && LODLevel->RequiredModule->bUseLocalSpace)
	{
		FTransform Mat = Context.GetTransform();
		FVector3f LocalAcceleration(Mat.InverseTransformVector(Acceleration));
		BEGIN_UPDATE_LOOP;
		{
			FPlatformMisc::Prefetch(ParticleData, (ParticleIndices[i+1] * ParticleStride));
			FPlatformMisc::Prefetch(ParticleData, (ParticleIndices[i+1] * ParticleStride) + PLATFORM_CACHE_LINE_SIZE);
			Particle.Velocity		+= LocalAcceleration * DeltaTime;
			Particle.BaseVelocity	+= LocalAcceleration * DeltaTime;
		}
		END_UPDATE_LOOP;
	}
	else
	{
		FVector3f LocalAcceleration(Acceleration);
		if (LODLevel->RequiredModule->bUseLocalSpace)
		{
			LocalAcceleration = FVector4f(Owner->EmitterToSimulation.TransformVector((FVector)LocalAcceleration));
		}
		BEGIN_UPDATE_LOOP;
		{
			FPlatformMisc::Prefetch(ParticleData, (ParticleIndices[i+1] * ParticleStride));
			FPlatformMisc::Prefetch(ParticleData, (ParticleIndices[i+1] * ParticleStride) + PLATFORM_CACHE_LINE_SIZE);
			Particle.Velocity		+= LocalAcceleration * DeltaTime;
			Particle.BaseVelocity	+= LocalAcceleration * DeltaTime;
		}
		END_UPDATE_LOOP;
	}
}

/*-----------------------------------------------------------------------------
	ParticleModuleAccelerationDrag implementation.
-----------------------------------------------------------------------------*/

UParticleModuleAccelerationDrag::UParticleModuleAccelerationDrag(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bSpawnModule = false;
	bUpdateModule = true;
}

void UParticleModuleAccelerationDrag::InitializeDefaults()
{
	if (!DragCoefficientRaw.IsCreated())
	{
		UDistributionFloatConstant* DistributionDragCoefficient = NewObject<UDistributionFloatConstant>(this, TEXT("DistributionDragCoefficient"));
		DistributionDragCoefficient->Constant = 1.0f;
		DragCoefficientRaw.Distribution = DistributionDragCoefficient;
	}
}

void UParticleModuleAccelerationDrag::PostInitProperties()
{
	Super::PostInitProperties();
	if (!HasAnyFlags(RF_ClassDefaultObject | RF_NeedLoad))
	{
		InitializeDefaults();
	}
}

void UParticleModuleAccelerationDrag::PostLoad()
{
	Super::PostLoad();
#if WITH_EDITOR
	if(DragCoefficient_DEPRECATED)
	{
		DragCoefficientRaw.Distribution = DragCoefficient_DEPRECATED;
		DragCoefficientRaw.Initialize();
	}
	DragCoefficient_DEPRECATED = nullptr;
#endif
}

void UParticleModuleAccelerationDrag::CompileModule(FParticleEmitterBuildInfo& EmitterInfo)
{
	EmitterInfo.DragCoefficient.Initialize(DragCoefficientRaw.Distribution);
}

#if WITH_EDITOR
void UParticleModuleAccelerationDrag::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	InitializeDefaults();
	Super::PostEditChangeProperty(PropertyChangedEvent);
}

bool UParticleModuleAccelerationDrag::IsValidForLODLevel(UParticleLODLevel* LODLevel, FString& OutErrorString)
{
	if (DragCoefficientRaw.Distribution && LODLevel->TypeDataModule && LODLevel->TypeDataModule->IsA(UParticleModuleTypeDataGpu::StaticClass()))
	{
		if(!IsDistributionAllowedOnGPU(DragCoefficientRaw.Distribution))
		{
			OutErrorString = GetDistributionNotAllowedOnGPUText(StaticClass()->GetName(), "DragCoefficient" ).ToString();
			return false;
		}
	}

	return true;
}
#endif // WITH_EDITOR

void UParticleModuleAccelerationDrag::Update(const FUpdateContext& Context)
{
	FParticleEmitterInstance* Owner = &Context.Owner;
	BEGIN_UPDATE_LOOP;
	{
		FVector3f Drag  = Particle.Velocity * -DragCoefficientRaw.GetValue(Particle.RelativeTime, Context.GetDistributionData());
		Particle.Velocity		+= Drag * DeltaTime;
		Particle.BaseVelocity	+= Drag * DeltaTime;
	}
	END_UPDATE_LOOP;
}

/*-----------------------------------------------------------------------------
	ParticleModuleAccelerationDragScaleOverLife implementation.
-----------------------------------------------------------------------------*/
UParticleModuleAccelerationDragScaleOverLife::UParticleModuleAccelerationDragScaleOverLife(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

void UParticleModuleAccelerationDragScaleOverLife::InitializeDefaults()
{
	if (!DragScaleRaw.IsCreated())
	{
		UDistributionFloatConstant* DistributionDragScale = NewObject<UDistributionFloatConstant>(this, TEXT("DistributionDragScale"));
		DistributionDragScale->Constant = 1.0f;
		DragScaleRaw.Distribution = DistributionDragScale;
	}
}

void UParticleModuleAccelerationDragScaleOverLife::PostInitProperties()
{
	Super::PostInitProperties();
	if (!HasAnyFlags(RF_ClassDefaultObject | RF_NeedLoad))
	{
		InitializeDefaults();
	}
}

void UParticleModuleAccelerationDragScaleOverLife::PostLoad()
{
	Super::PostLoad();
#if WITH_EDITOR
	if (DragScale_DEPRECATED)
	{
		DragScaleRaw.Distribution = DragScale_DEPRECATED;
		DragScaleRaw.Initialize();
	}
	DragScale_DEPRECATED = nullptr;
#endif
}

void UParticleModuleAccelerationDragScaleOverLife::CompileModule(FParticleEmitterBuildInfo& EmitterInfo)
{
	EmitterInfo.DragScale.ScaleByDistribution(DragScaleRaw.Distribution);
}

#if WITH_EDITOR
void UParticleModuleAccelerationDragScaleOverLife::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	InitializeDefaults();
	Super::PostEditChangeProperty(PropertyChangedEvent);
}

bool UParticleModuleAccelerationDragScaleOverLife::IsValidForLODLevel(UParticleLODLevel* LODLevel, FString& OutErrorString)
{
	if (LODLevel->TypeDataModule && LODLevel->TypeDataModule->IsA(UParticleModuleTypeDataGpu::StaticClass()))
	{
		if(!IsDistributionAllowedOnGPU(DragScaleRaw.Distribution))
		{
			OutErrorString = GetDistributionNotAllowedOnGPUText(StaticClass()->GetName(), "DragScale" ).ToString();
			return false;
		}
	}

	return true;
}

#endif // WITH_EDITOR

/*-----------------------------------------------------------------------------
	ParticleModuleAttractorPointGravity implementation.
-----------------------------------------------------------------------------*/
UParticleModuleAttractorPointGravity::UParticleModuleAttractorPointGravity(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bSupported3DDrawMode = true;
}

void UParticleModuleAttractorPointGravity::InitializeDefaults()
{
	if (!StrengthRaw.IsCreated())
	{
		UDistributionFloatConstant* DistributionStrength = NewObject<UDistributionFloatConstant>(this, TEXT("DistributionStrength"));
		DistributionStrength->Constant = 1.0f;
		StrengthRaw.Distribution = DistributionStrength;
	}
}

void UParticleModuleAttractorPointGravity::PostInitProperties()
{
	Super::PostInitProperties();
	if (!HasAnyFlags(RF_ClassDefaultObject | RF_NeedLoad))
	{
		InitializeDefaults();
	}
}

void UParticleModuleAttractorPointGravity::PostLoad()
{
	Super::PostLoad();
#if WITH_EDITOR
	if (Strength_DEPRECATED)
	{
		StrengthRaw.Distribution = Strength_DEPRECATED;
		StrengthRaw.Initialize();
	}
	Strength_DEPRECATED = nullptr;
#endif
}

void UParticleModuleAttractorPointGravity::CompileModule(FParticleEmitterBuildInfo& EmitterInfo)
{
	EmitterInfo.PointAttractorPosition = Position;
	EmitterInfo.PointAttractorRadius = Radius;
	EmitterInfo.PointAttractorStrength.Initialize(StrengthRaw.Distribution);
}

#if WITH_EDITOR
void UParticleModuleAttractorPointGravity::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	InitializeDefaults();
	Super::PostEditChangeProperty(PropertyChangedEvent);
}
#endif // WITH_EDITOR

void UParticleModuleAttractorPointGravity::Render3DPreview(const FPreviewContext& Context)
{
#if WITH_EDITOR
	FPrimitiveDrawInterface* PDI = Context.PDI;

	// Draw a wire star at the position.
	DrawWireStar(PDI,Position, 10.0f, ModuleEditorColor, SDPG_World);

	// Draw bounding circles for the range.
	DrawCircle(PDI,Position, FVector(1,0,0), FVector(0,1,0), ModuleEditorColor, Radius, 32, SDPG_World);
	DrawCircle(PDI,Position, FVector(1,0,0), FVector(0,0,1), ModuleEditorColor, Radius, 32, SDPG_World);
	DrawCircle(PDI,Position, FVector(0,1,0), FVector(0,0,1), ModuleEditorColor, Radius, 32, SDPG_World);
#endif	//#if WITH_EDITOR
}

/*-----------------------------------------------------------------------------
	UParticleModuleAcceleration implementation.
-----------------------------------------------------------------------------*/
UParticleModuleAcceleration::UParticleModuleAcceleration(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bSpawnModule = true;
	bUpdateModule = true;
}

void UParticleModuleAcceleration::InitializeDefaults()
{
	if (!Acceleration.IsCreated())
	{
		Acceleration.Distribution = NewObject<UDistributionVectorUniform>(this, TEXT("DistributionAcceleration"));
	}
}

void UParticleModuleAcceleration::PostInitProperties()
{
	Super::PostInitProperties();
	if (!HasAnyFlags(RF_ClassDefaultObject | RF_NeedLoad))
	{
		InitializeDefaults();
	}
}

void UParticleModuleAcceleration::CompileModule( FParticleEmitterBuildInfo& EmitterInfo )
{
	EmitterInfo.ConstantAcceleration = Acceleration.GetValue();
}

#if WITH_EDITOR
void UParticleModuleAcceleration::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	InitializeDefaults();
	Super::PostEditChangeProperty(PropertyChangedEvent);
}
#endif // WITH_EDITOR

const FTransform& UParticleModuleAcceleration::FContext::GetTransform() const
{
	return Owner.Component.GetComponentTransform();
}

UObject* UParticleModuleAcceleration::FContext::GetDistributionData() const
{
	return Owner.Component.GetDistributionData();
}

FString UParticleModuleAcceleration::FContext::GetTemplateName() const
{
	{
		return GetNameSafe(Owner.Component.GetTemplate());
	}
}

FString UParticleModuleAcceleration::FContext::GetInstanceName() const
{
	return Owner.Component.GetPathName();
}

void UParticleModuleAcceleration::Spawn(const FSpawnContext& Context)
{
	FParticleEmitterInstance& Owner = Context.Owner;
	SPAWN_INIT;
	PARTICLE_ELEMENT(FVector3f, UsedAcceleration);
	UsedAcceleration = FVector3f(Acceleration.GetValue(Owner.EmitterTime, Context.GetDistributionData()));
	if (bApplyOwnerScale == true)
	{
		FVector3f Scale(Context.GetTransform().GetScale3D());
		UsedAcceleration *= Scale;
	}
	UParticleLODLevel* LODLevel	= Owner.SpriteTemplate->GetCurrentLODLevel(&Owner);
	check(LODLevel);
	if (bAlwaysInWorldSpace && LODLevel->RequiredModule->bUseLocalSpace)
	{
		FVector3f TempUsedAcceleration = (FVector3f)Context.GetTransform().InverseTransformVector(FVector(UsedAcceleration));
		Particle.Velocity		+= TempUsedAcceleration * Context.SpawnTime;
		Particle.BaseVelocity	+= TempUsedAcceleration * Context.SpawnTime;
	}
	else
	{
		if (LODLevel->RequiredModule->bUseLocalSpace)
		{
			UsedAcceleration = (FVector4f)Owner.EmitterToSimulation.TransformVector(FVector(UsedAcceleration));
		}
		Particle.Velocity		+= UsedAcceleration * Context.SpawnTime;
		Particle.BaseVelocity	+= UsedAcceleration * Context.SpawnTime;
	}
}

void UParticleModuleAcceleration::Update(const FUpdateContext& Context)
{
	FParticleEmitterInstance* Owner = &Context.Owner;
	if ((Owner == NULL) || (Owner->ActiveParticles <= 0) || 
		(Owner->ParticleData == NULL) || (Owner->ParticleIndices == NULL))
	{
		return;
	}
	UParticleLODLevel* LODLevel	= Owner->SpriteTemplate->GetCurrentLODLevel(Owner);
	check(LODLevel);
	FPlatformMisc::Prefetch(Owner->ParticleData, (Owner->ParticleIndices[0] * Owner->ParticleStride));
	FPlatformMisc::Prefetch(Owner->ParticleData, (Owner->ParticleIndices[0] * Owner->ParticleStride) + PLATFORM_CACHE_LINE_SIZE);
	if (bAlwaysInWorldSpace && LODLevel->RequiredModule->bUseLocalSpace)
	{
		FTransform Mat = Context.GetTransform();
		BEGIN_UPDATE_LOOP;
		{
			FVector3f& UsedAcceleration = *((FVector3f*)(ParticleBase + CurrentOffset));																\
			FVector3f TransformedUsedAcceleration = (FVector3f)Mat.InverseTransformVector((FVector)UsedAcceleration);
			FPlatformMisc::Prefetch(ParticleData, (ParticleIndices[i+1] * ParticleStride));
			FPlatformMisc::Prefetch(ParticleData, (ParticleIndices[i+1] * ParticleStride) + PLATFORM_CACHE_LINE_SIZE);
			Particle.Velocity		+= TransformedUsedAcceleration * DeltaTime;
			Particle.BaseVelocity	+= TransformedUsedAcceleration * DeltaTime;
		}
		END_UPDATE_LOOP;
	}
	else
	{
		BEGIN_UPDATE_LOOP;
		{
			FVector3f& UsedAcceleration = *((FVector3f*)(ParticleBase + CurrentOffset));																\
			FPlatformMisc::Prefetch(ParticleData, (ParticleIndices[i+1] * ParticleStride));
			FPlatformMisc::Prefetch(ParticleData, (ParticleIndices[i+1] * ParticleStride) + PLATFORM_CACHE_LINE_SIZE);
			Particle.Velocity		+= UsedAcceleration * DeltaTime;
			Particle.BaseVelocity	+= UsedAcceleration * DeltaTime;
		}
		END_UPDATE_LOOP;
	}
}

uint32 UParticleModuleAcceleration::RequiredBytes(UParticleModuleTypeDataBase* TypeData)
{
	// FVector3f UsedAcceleration
	return sizeof(FVector3f);
}

#if WITH_EDITOR
bool UParticleModuleAcceleration::IsValidForLODLevel(UParticleLODLevel* LODLevel, FString& OutErrorString)
{
	if (LODLevel->TypeDataModule && LODLevel->TypeDataModule->IsA(UParticleModuleTypeDataGpu::StaticClass()))
	{
		if(!IsDistributionAllowedOnGPU(Acceleration.Distribution))
		{
			OutErrorString = GetDistributionNotAllowedOnGPUText(StaticClass()->GetName(), "Acceleration" ).ToString();
			return false;
		}
	}

	return true;
}
#endif

/*-----------------------------------------------------------------------------
	UParticleModuleAccelerationOverLifetime implementation.
-----------------------------------------------------------------------------*/

UParticleModuleAccelerationOverLifetime::UParticleModuleAccelerationOverLifetime(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bSpawnModule = false;
	bUpdateModule = true;
}

void UParticleModuleAccelerationOverLifetime::InitializeDefaults()
{
	if (!AccelOverLife.IsCreated())
	{
		AccelOverLife.Distribution = NewObject<UDistributionVectorConstantCurve>(this, TEXT("DistributionAccelOverLife"));
	}
}

void UParticleModuleAccelerationOverLifetime::PostInitProperties()
{
	Super::PostInitProperties();
	if (!HasAnyFlags(RF_ClassDefaultObject | RF_NeedLoad))
	{
		InitializeDefaults();
	}
}

#if WITH_EDITOR
void UParticleModuleAccelerationOverLifetime::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	InitializeDefaults();
	Super::PostEditChangeProperty(PropertyChangedEvent);
}
#endif // WITH_EDITOR

void UParticleModuleAccelerationOverLifetime::Update(const FUpdateContext& Context)
{
	FParticleEmitterInstance* Owner = &Context.Owner;
	UParticleLODLevel* LODLevel	= Owner->SpriteTemplate->GetCurrentLODLevel(Owner);
	check(LODLevel);
	if (bAlwaysInWorldSpace && LODLevel->RequiredModule->bUseLocalSpace)
	{
		FTransform Mat = Context.GetTransform();
		BEGIN_UPDATE_LOOP;
			// Acceleration should always be in world space...
			FVector Accel = AccelOverLife.GetValue(Particle.RelativeTime, Context.GetDistributionData());
			Accel = Mat.InverseTransformVector(Accel);
			Particle.Velocity		+= (FVector3f)Accel * DeltaTime;
			Particle.BaseVelocity	+= (FVector3f)Accel * DeltaTime;
		END_UPDATE_LOOP;
	}
	else
	{
		BEGIN_UPDATE_LOOP;
		// Acceleration should always be in world space...
		FVector Accel = AccelOverLife.GetValue(Particle.RelativeTime, Context.GetDistributionData());
		Particle.Velocity		+= (FVector3f)Accel * DeltaTime;
		Particle.BaseVelocity	+= (FVector3f)Accel * DeltaTime;
		END_UPDATE_LOOP;
	}
}

/*-----------------------------------------------------------------------------
	UParticleModuleLight implementation.
-----------------------------------------------------------------------------*/

UParticleModuleLightBase::UParticleModuleLightBase(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

UParticleModuleLight::UParticleModuleLight(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bSpawnModule = true;
	bUpdateModule = true;
	bCurvesAsColor = true;
	bUseInverseSquaredFalloff = true;
	SpawnFraction = 1;
	bSupported3DDrawMode = true;
	b3DDrawMode = true;
	// Particle lights don't affect volumetric fog by default, since they are likely to be moving fast and volumetric fog relies on a strong temporal filter
	VolumetricScatteringIntensity = 0.0f;
}

void UParticleModuleLight::InitializeDefaults()
{
	if (!ColorScaleOverLife.IsCreated())
	{
		ColorScaleOverLife.Distribution = NewObject<UDistributionVectorConstant>(this, TEXT("DistributionColorScaleOverLife"));
	}

	if (!BrightnessOverLife.IsCreated())
	{
		BrightnessOverLife.Distribution = NewObject<UDistributionFloatConstant>(this, TEXT("DistributionBrightnessOverLife"));
	}

	if (!RadiusScale.IsCreated())
	{
		RadiusScale.Distribution = NewObject<UDistributionFloatConstant>(this, TEXT("DistributionRadiusScale"));
	}

	if (!LightExponent.IsCreated())
	{
		LightExponent.Distribution = NewObject<UDistributionFloatConstant>(this, TEXT("DistributionLightExponent"));
	}

	if (!bHighQualityLights)
	{
		bShadowCastingLights = false;
	}
}

void UParticleModuleLight::PostInitProperties()
{
	Super::PostInitProperties();

	if (!HasAnyFlags(RF_ClassDefaultObject | RF_NeedLoad))
	{
		InitializeDefaults();
	}
}

#if WITH_EDITOR
void UParticleModuleLight::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	InitializeDefaults();
	Super::PostEditChangeProperty(PropertyChangedEvent);
}
#endif // WITH_EDITOR

bool UParticleModuleLight::CanTickInAnyThread()
{
	return !bHighQualityLights && BrightnessOverLife.OkForParallel() && ColorScaleOverLife.OkForParallel() && RadiusScale.OkForParallel() && LightExponent.OkForParallel();
}

static TAutoConsoleVariable<int32> CVarParticleLightQuality(
	TEXT("r.ParticleLightQuality"),
	2,
	TEXT("0: No lights. 1:Only simple lights. 2:Simple+HQ lights"),
	ECVF_Scalability
	);

static TAutoConsoleVariable<int32> CVarParticleDefaultLightInverseExposureBlend(
	TEXT("fx.ParticleDefaultLightInverseExposureBlend"),
	0.0f,
	TEXT("Blend Factor used to blend between Intensity and Intensity / Exposure."),
	ECVF_Default
);

void UParticleModuleLight::SpawnEx(const FSpawnContext& Context, struct FRandomStream* InRandomStream)
{
	int32 ParticleLightQuality = CVarParticleLightQuality.GetValueOnAnyThread();
	if (ParticleLightQuality > 0)
	{
		FParticleEmitterInstance* Owner = &Context.Owner;
		SPAWN_INIT;
		PARTICLE_ELEMENT(FLightParticlePayload, LightData);
		const float Brightness = BrightnessOverLife.GetValue(Particle.RelativeTime, Context.GetDistributionData(), InRandomStream);
		LightData.ColorScale = (FVector3f)ColorScaleOverLife.GetValue(Particle.RelativeTime, Context.GetDistributionData(), 0, InRandomStream) * Brightness;
		LightData.RadiusScale = RadiusScale.GetValue(Owner->EmitterTime, Context.GetDistributionData(), InRandomStream);
		// Exponent of 0 is interpreted by renderer as inverse squared falloff
		LightData.LightExponent = bUseInverseSquaredFalloff ? 0 : LightExponent.GetValue(Owner->EmitterTime, Context.GetDistributionData(), InRandomStream);
		LightData.InverseExposureBlend = bOverrideInverseExposureBlend ? InverseExposureBlend : CVarParticleDefaultLightInverseExposureBlend.GetValueOnAnyThread();
		const float RandomNumber = InRandomStream->GetFraction();
		LightData.bValid = RandomNumber < SpawnFraction;
		LightData.bAffectsTranslucency = bAffectsTranslucency;
		LightData.bHighQuality = bHighQualityLights;
		LightData.LightId = 0;

		if (bHighQualityLights && ParticleLightQuality > 1 && LightData.bValid)
		{		
			LightData.LightId = SpawnHQLight(LightData, Particle, Owner);
		}
	}
}

uint64 UParticleModuleLight::SpawnHQLight(const FLightParticlePayload& Payload, const FBaseParticle& Particle, FParticleEmitterInstance* Owner)
{
	uint64 LightId = 0;
	if (!Owner)
	{
		return 0;
	}
	UParticleSystemComponent* ParticleSystem = Owner->Component.AsComponent();
	if (!ParticleSystem)
	{
		return 0;
	}
	AActor* HQLightContainer = ParticleSystem->GetOwner();
	if (!HQLightContainer || HQLightContainer->IsPendingKillPending())
	{
		return 0;
	}

	// Construct the new component and attach as needed				
	UPointLightComponent* PointLightComponent = NewObject<UPointLightComponent>(HQLightContainer, NAME_None, RF_Transient);
	if (PointLightComponent)
	{
		LightId = (uint64)PointLightComponent;
						
		USceneComponent* RootComponent = HQLightContainer->GetRootComponent();
		USceneComponent* AttachParent = ParticleSystem->GetAttachParent();
		if (AttachParent)
		{			
			PointLightComponent->SetupAttachment(AttachParent, ParticleSystem->GetAttachSocketName());
		}
		else if (RootComponent)
		{
			PointLightComponent->SetupAttachment(RootComponent);
		}			
		PointLightComponent->CreationMethod = ParticleSystem->CreationMethod;
		PointLightComponent->LightingChannels = LightingChannels;
		PointLightComponent->bUseInverseSquaredFalloff = bUseInverseSquaredFalloff;
		PointLightComponent->bAffectTranslucentLighting = bAffectsTranslucency;
		PointLightComponent->VolumetricScatteringIntensity = VolumetricScatteringIntensity;
		PointLightComponent->SetCastShadows(bShadowCastingLights);		

		PointLightComponent->RegisterComponent();
		Owner->HighQualityLights.Add(PointLightComponent);

		int32 ScreenAlignment;
		FVector ComponentScale;
		Owner->GetScreenAlignmentAndScale(ScreenAlignment, ComponentScale);
		UpdateHQLight(PointLightComponent, Payload, Particle, ScreenAlignment, ComponentScale, Owner->UseLocalSpace(), nullptr, false);
	}	
	return LightId;
}

void UParticleModuleLight::UpdateHQLight(UPointLightComponent* PointLightComponent, const FLightParticlePayload& Payload, const FBaseParticle& Particle, int32 ScreenAlignment, FVector ComponentScale, bool bLocalSpace, FSceneInterface* OwnerScene, bool bDoRTUpdate)
{
	if (bLocalSpace)
	{
		PointLightComponent->SetRelativeLocation(Particle.Location);
	}
	else
	{
		PointLightComponent->SetWorldLocation(Particle.Location);
	}
	
	FLinearColor DesiredFinalColor = FLinearColor(FVector3f(Particle.Color) * Particle.Color.A * Payload.ColorScale);
	if (bUseInverseSquaredFalloff)
	{
		// For compatibility reasons, the default units are ELightUnits::Unitless. If this change, this needs to be updated.
		ensure(PointLightComponent->IntensityUnits == ELightUnits::Unitless);

		// Non HQ lights are drawn in 0.0001 Candelas units according to the lighting in DeferredLightPixelShaders.usf
		// the 1/100^2 factor comes from the radial attenuation being computed in cm instead of in meters.
		static const float ShaderUnitConversion  = UPointLightComponent::GetUnitsConversionFactor(ELightUnits::Candelas, ELightUnits::Unitless) / (100.f * 100.f);
		DesiredFinalColor *= ShaderUnitConversion;
	}

	//light color on HQ lights is just a uint32 and our light scalars can be huge.  To preserve the color control and range from the particles we need to normalize
	//around the full range multiplied value, and set the scalar intensity such that it will bring things back into line later.
	FVector3f AdjustedColor(DesiredFinalColor.R, DesiredFinalColor.G, DesiredFinalColor.B);
	float Intensity = AdjustedColor.Size();
	AdjustedColor.Normalize();	
	
	//light module currently needs to run AFTER any size modification modules to get a value that matches 'simple' lights.
	FVector2D Size;
	Size.X = FMath::Abs(Particle.Size.X * ComponentScale.X);
	Size.Y = FMath::Abs(Particle.Size.Y * ComponentScale.Y);
	if (ScreenAlignment == PSA_Square || ScreenAlignment == PSA_FacingCameraPosition || ScreenAlignment == PSA_FacingCameraDistanceBlend)
	{
		Size.Y = Size.X;
	}
	float Radius = Payload.RadiusScale * (Size.X + Size.X) * 0.5f;

	//HQ light color is an FColor which is a uint32.  Thus we have to break out the out of range value into the intensity scalar.
	FColor NormalizedColor;
	NormalizedColor.R = (uint8)FMath::Min(AdjustedColor.X * 255.0f, 255.0f);
	NormalizedColor.G = (uint8)FMath::Min(AdjustedColor.Y * 255.0f, 255.0f);
	NormalizedColor.B = (uint8)FMath::Min(AdjustedColor.Z * 255.0f, 255.0f);
	PointLightComponent->SetIntensity(Intensity);
	PointLightComponent->SetLightColor(NormalizedColor);
	PointLightComponent->SetAttenuationRadius(Radius);
	PointLightComponent->SetLightFalloffExponent(Payload.LightExponent);
	PointLightComponent->SetInverseExposureBlend(Payload.InverseExposureBlend);

	if (OwnerScene && bDoRTUpdate)
	{
		OwnerScene->UpdateLightTransform(PointLightComponent);
		OwnerScene->UpdateLightColorAndBrightness(PointLightComponent);
	}
}

void UParticleModuleLight::Spawn(const FSpawnContext& Context)
{
	SpawnEx(Context, &GetRandomStream(Context));
}

void UParticleModuleLight::Update(const FUpdateContext& Context)
{
	FParticleEmitterInstance* Owner = &Context.Owner;
	if ((Owner == NULL) || (Owner->ActiveParticles <= 0) || 
		(Owner->ParticleData == NULL) || (Owner->ParticleIndices == NULL))
	{
		return;
	}

	FSceneInterface* OwnerScene = Owner->Component.GetScene();;
	
	TSet<uint64> ActiveLights;
	UParticleLODLevel* LODLevel	= Owner->SpriteTemplate->GetCurrentLODLevel(Owner);
	check(LODLevel);
	FPlatformMisc::Prefetch(Owner->ParticleData, (Owner->ParticleIndices[0] * Owner->ParticleStride));
	FPlatformMisc::Prefetch(Owner->ParticleData, (Owner->ParticleIndices[0] * Owner->ParticleStride) + PLATFORM_CACHE_LINE_SIZE);
	const bool bUseLocalSpace = Owner->UseLocalSpace();
	int32 ScreenAlignment;
	FVector ComponentScale;
	Owner->GetScreenAlignmentAndScale(ScreenAlignment, ComponentScale);

	BEGIN_UPDATE_LOOP;
	{
		PARTICLE_ELEMENT(FLightParticlePayload,	Data);
		const float Brightness = BrightnessOverLife.GetValue(Particle.RelativeTime, Context.GetDistributionData());
		Data.ColorScale = (FVector3f)ColorScaleOverLife.GetValue(Particle.RelativeTime, Context.GetDistributionData()) * Brightness;

		if (bHighQualityLights && (Data.LightId != 0))
		{
			ActiveLights.Add(Data.LightId);

			//for now we can do this
			UPointLightComponent* PointLightComponent = (UPointLightComponent*)Data.LightId;
			UpdateHQLight(PointLightComponent, Data, Particle, ScreenAlignment, ComponentScale, bUseLocalSpace, OwnerScene, true);
		}
	}
	END_UPDATE_LOOP;

	//remove any dead lights.
	if (bHighQualityLights)
	{
		for (int32 i = 0; i < Owner->HighQualityLights.Num(); ++i)
		{
			UPointLightComponent* PointLightComponent = Owner->HighQualityLights[i];
			if (PointLightComponent && !ActiveLights.Contains((uint64)PointLightComponent))
			{
				PointLightComponent->Modify();
				PointLightComponent->DestroyComponent(false);
				Owner->HighQualityLights.RemoveAtSwap(i);
				--i;
			}
		}
	}
}

uint32 UParticleModuleLight::RequiredBytes(UParticleModuleTypeDataBase* TypeData)
{
	return sizeof(FLightParticlePayload);
}

void UParticleModuleLight::SetToSensibleDefaults(UParticleEmitter* Owner)
{
	UDistributionVectorConstant* ColorScaleDist = Cast<UDistributionVectorConstant>(ColorScaleOverLife.Distribution);
	if (ColorScaleDist)
	{
		ColorScaleDist->Constant = FVector(1, 1, 1);
		ColorScaleDist->bIsDirty = true;
	}

	UDistributionFloatConstant* BrightnessDist = Cast<UDistributionFloatConstant>(BrightnessOverLife.Distribution);
	if (BrightnessDist)
	{
		BrightnessDist->Constant = 32.0f;
		BrightnessDist->bIsDirty = true;
	}


	UDistributionFloatConstant* RadiusScaleDist = Cast<UDistributionFloatConstant>(RadiusScale.Distribution);
	if (RadiusScaleDist)
	{
		RadiusScaleDist->Constant = 15.0f;
		RadiusScaleDist->bIsDirty = true;
	}

	UDistributionFloatConstant* LightExponentDist = Cast<UDistributionFloatConstant>(LightExponent.Distribution);
	if (LightExponentDist)
	{
		LightExponentDist->Constant = 16.0f;
		LightExponentDist->bIsDirty = true;
	}
}

void UParticleModuleLight::Render3DPreview(const FPreviewContext& Context)
{
#if WITH_EDITOR
	FParticleEmitterInstance* Owner = &Context.Owner;
	if ((Owner == NULL) || (Owner->ActiveParticles <= 0) || 
		(Owner->ParticleData == NULL) || (Owner->ParticleIndices == NULL))
	{
		return;
	}

	if (bPreviewLightRadius)
	{
		check(IsInGameThread());
		int32 Offset = 0;
		UParticleLODLevel* LODLevel	= Owner->SpriteTemplate->GetCurrentLODLevel(Owner);
		const bool bLocalSpace = LODLevel->RequiredModule->bUseLocalSpace;
		const FVector Scale = Context.GetTransform().GetScale3D();
		const FMatrix LocalToWorld = Owner->EmitterToSimulation * Owner->SimulationToWorld;
		check(LODLevel);

		const uint8* ParticleData = Owner->ParticleData;
		const uint16* ParticleIndices = Owner->ParticleIndices;

		FPrimitiveDrawInterface* PDI = Context.PDI;

		for (int32 i = 0; i < Owner->ActiveParticles; i++)
		{
			DECLARE_PARTICLE_CONST(Particle, ParticleData + Owner->ParticleStride * ParticleIndices[i]);

			const FLightParticlePayload* LightPayload = (const FLightParticlePayload*)(((const uint8*)&Particle) + Owner->LightDataOffset);

			if (LightPayload->bValid)
			{
				const FVector LightPosition = bLocalSpace ? FVector(LocalToWorld.TransformPosition(Particle.Location)) : FVector(Particle.Location);
				const FVector Size = Scale * (FVector)Particle.Size;
				const float LightRadius = LightPayload->RadiusScale * (Size.X + Size.Y) / 2.0f;

				DrawWireSphere(PDI, LightPosition, FColor::White, LightRadius, 18, SDPG_World);
			}
		}
	}

#endif	//#if WITH_EDITOR
}

UParticleModuleLight_Seeded::UParticleModuleLight_Seeded(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bSpawnModule = true;
	bSupportsRandomSeed = true;
	bRequiresLoopingNotification = true;
}

void UParticleModuleLight_Seeded::EmitterLoopingNotify(FParticleEmitterInstance* Owner)
{
	if (RandomSeedInfo.bResetSeedOnEmitterLooping == true)
	{
		FParticleRandomSeedInstancePayload* Payload = Owner->GetModuleRandomSeedInstanceData(this);
		PrepRandomSeedInstancePayload(Owner, Payload, RandomSeedInfo);
	}
}

/*-----------------------------------------------------------------------------
	UParticleModuleTypeDataBase implementation.
-----------------------------------------------------------------------------*/
UParticleModuleTypeDataBase::UParticleModuleTypeDataBase(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bSpawnModule = false;
	bUpdateModule = false;
}

FParticleEmitterInstance* UParticleModuleTypeDataBase::CreateInstance(UParticleEmitter* InEmitterParent, IParticleEmitterInstanceOwner& InComponent)
{
	return NULL;
}

/*-----------------------------------------------------------------------------
	UParticleModuleTypeDataMesh implementation.
-----------------------------------------------------------------------------*/
static TAutoConsoleVariable<int32> CVarMinDetailModeForMeshParticleMotionBlur(
	TEXT("r.MeshParticle.MinDetailModeForMotionBlur"),
	-1,
	TEXT("Sets the minimum detail mode before mesh particles emit motion blur (Low  = 0, Med = 1, High = 2, Max = 3). ")
	TEXT("Set to -1 to disable mesh particles motion blur entirely. Defaults to -1.")
	);

int32 UParticleModuleTypeDataMesh::GetCurrentDetailMode()
{
	return GetCachedScalabilityCVars().DetailMode;
}

int32 UParticleModuleTypeDataMesh::GetMeshParticleMotionBlurMinDetailMode()
{
	return CVarMinDetailModeForMeshParticleMotionBlur.GetValueOnAnyThread();
}

UParticleModuleTypeDataMesh::UParticleModuleTypeDataMesh(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	CastShadows = false;
	DoCollisions = false;
	MeshAlignment = PSMA_MeshFaceCameraWithRoll;
	AxisLockOption = EPAL_NONE;
	CameraFacingUpAxisOption_DEPRECATED = CameraFacing_NoneUP;
	CameraFacingOption = XAxisFacing_NoUp;
	bCollisionsConsiderPartilceSize = true;
	bUseStaticMeshLODs = true;
	LODSizeScale = 1.0f;
}

FParticleEmitterInstance* UParticleModuleTypeDataMesh::CreateInstance(UParticleEmitter* InEmitterParent, IParticleEmitterInstanceOwner& InComponent)
{
	SetToSensibleDefaults(InEmitterParent);
	FParticleEmitterInstance* Instance = new FParticleMeshEmitterInstance(InComponent);
	check(Instance);

	Instance->InitParameters(InEmitterParent);

	CreateDistribution();

#if WITH_EDITOR
	if (GIsEditor && (Mesh != nullptr))
	{
		Mesh->GetOnMeshChanged().AddUObject(this, &UParticleModuleTypeDataMesh::OnMeshChanged);
	}
#endif

	return Instance;
}

const FVertexFactoryType* UParticleModuleTypeDataMesh::GetVertexFactoryType() const
{
	return &FMeshParticleVertexFactory::StaticType;
}

extern void InitMeshParticleVertexFactoryComponents(FMeshParticleVertexFactory* InVertexFactory, const FStaticMeshLODResources& LODResources, FMeshParticleVertexFactory::FDataType& Data);

void UParticleModuleTypeDataMesh::CollectPSOPrecacheData(const UParticleEmitter* Emitter, FPSOPrecacheParams& OutParams)
{
	if (Mesh != nullptr)
	{
		FStaticMeshRenderData* RenderData = Mesh->GetRenderData();
		// Assuming here that all LOD use same vertex decl
		int32 MeshLODIdx = Mesh->GetMinLODIdx();
		if (RenderData->LODResources.IsValidIndex(MeshLODIdx))
		{
			bool bUsesDynamicParameter = (Emitter->DynamicParameterDataOffset > 0);
			int32 DynamicVertexStride = sizeof(FMeshParticleInstanceVertex);
			int32 DynamicParameterVertexStride = bUsesDynamicParameter ? sizeof(FMeshParticleInstanceVertexDynamicParameter) : 0;
						
			FVertexDeclarationElementList Elements;
			FMeshParticleVertexFactory::FDataType Data;
			InitMeshParticleVertexFactoryComponents(nullptr, RenderData->LODResources[MeshLODIdx], Data);
			FMeshParticleVertexFactory::GetVertexElements(GMaxRHIFeatureLevel, DynamicVertexStride, DynamicParameterVertexStride, Data, Elements);
			const FVertexFactoryType* VFType = &FMeshParticleVertexFactory::StaticType;
			OutParams.PrimitiveType = GetPrimitiveType();
			OutParams.VertexFactoryDataList.Add(FPSOPrecacheVertexFactoryData(VFType, Elements));
		}
	}
}

void UParticleModuleTypeDataMesh::SetToSensibleDefaults(UParticleEmitter* Owner)
{
	if ((Mesh == NULL) && GIsEditor )
	{
		Mesh = (UStaticMesh*)StaticLoadObject(UStaticMesh::StaticClass(),NULL,TEXT("/Engine/EngineMeshes/ParticleCube.ParticleCube"),NULL,LOAD_None,NULL);
	}
}

void UParticleModuleTypeDataMesh::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);
	if (Ar.IsLoading() && Ar.UEVer() < VER_UE4_MESH_EMITTER_INITIAL_ORIENTATION_DISTRIBUTION)
	{
		FVector oldOrient(0.0f, 0.0f, 0.0f);
		CreateDistribution();
		UDistributionVectorUniform* RPYDistribution = Cast<UDistributionVectorUniform>(RollPitchYawRange.Distribution);
		RPYDistribution->Min = oldOrient;
		RPYDistribution->Max = oldOrient;
		RPYDistribution->bIsDirty = true;
	}
	if (Ar.IsLoading() && Ar.UEVer() < VER_UE4_MESH_PARTICLE_COLLISIONS_CONSIDER_PARTICLE_SIZE)
	{
		bCollisionsConsiderPartilceSize = false;//Old data should default to the old behavior of not considering particle size.
	}
}

void UParticleModuleTypeDataMesh::CreateDistribution()
{
	if (!RollPitchYawRange.IsCreated())
	{
		RollPitchYawRange.Distribution = NewObject<UDistributionVectorUniform>(this, TEXT("DistributionRollPitchYaw"));
	}
}


void UParticleModuleTypeDataMesh::PostLoad()
{
	Super::PostLoad();

	if (Mesh != nullptr)
	{
		Mesh->ConditionalPostLoad();
#if WITH_EDITOR
		if ( GIsEditor )
		{
			Mesh->GetOnMeshChanged().AddUObject(this, &UParticleModuleTypeDataMesh::OnMeshChanged);
		}
#endif
	}
}

bool UParticleModuleTypeDataMesh::IsPostLoadThreadSafe() const
{
	return false;
}

#if WITH_EDITOR
void UParticleModuleTypeDataMesh::OnMeshChanged()
{
	UObject* OuterObj = GetOuter();
	check(OuterObj);
	UParticleLODLevel* LODLevel = Cast<UParticleLODLevel>(OuterObj);
	if (LODLevel)
	{
		// The outer is incorrect - warn the user and handle it
		UE_LOGF(LogParticles, Warning, "UParticleModuleTypeDataMesh has an incorrect outer... run FixupEmitters on package %ls",
			*(OuterObj->GetOutermost()->GetPathName()));
		OuterObj = LODLevel->GetOuter();
		UParticleEmitter* Emitter = Cast<UParticleEmitter>(OuterObj);
		check(Emitter);
		OuterObj = Emitter->GetOuter();
	}

	FProperty* MeshProperty = FindFProperty<FProperty>(UParticleModuleTypeDataMesh::StaticClass(), FName("Mesh"));
	FPropertyChangedEvent PropertyChangedEvent(MeshProperty);

	UParticleSystem* PartSys = CastChecked<UParticleSystem>(OuterObj);
	PartSys->PostEditChangeProperty(PropertyChangedEvent);
}

void UParticleModuleTypeDataMesh::PreEditChange(FProperty* PropertyThatWillChange)
{
	Super::PreEditChange(PropertyThatWillChange);
	if ( (PropertyThatWillChange != nullptr) && (PropertyThatWillChange->GetFName() == FName("Mesh")) )
	{
		if ( GIsEditor && (Mesh != nullptr) )
		{
			Mesh->GetOnMeshChanged().RemoveAll(this);
		}
	}
}

void UParticleModuleTypeDataMesh::BeginDestroy()
{
	Super::BeginDestroy();
	if (GIsEditor && (Mesh != nullptr))
	{
		Mesh->GetOnMeshChanged().RemoveAll(this);
	}
}

void UParticleModuleTypeDataMesh::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	FProperty* PropertyThatChanged = PropertyChangedEvent.Property;
	if (PropertyThatChanged)
	{
		if (PropertyThatChanged->GetFName() == FName("Mesh"))
		{
			OnMeshChanged();
		}
		if (GIsEditor && (Mesh != nullptr))
		{
			Mesh->GetOnMeshChanged().AddUObject(this, &UParticleModuleTypeDataMesh::OnMeshChanged);
		}
	}
	Super::PostEditChangeProperty(PropertyChangedEvent);
}
#endif // WITH_EDITOR

UParticleModuleKillBase::UParticleModuleKillBase(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

/*-----------------------------------------------------------------------------
	UParticleModuleKillBox implementation.
-----------------------------------------------------------------------------*/
UParticleModuleKillBox::UParticleModuleKillBox(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bUpdateModule = true;
	bSupported3DDrawMode = true;
	bAxisAlignedAndFixedSize = true;
}

void UParticleModuleKillBox::InitializeDefaults()
{
	if (!LowerLeftCorner.IsCreated())
	{
		LowerLeftCorner.Distribution = NewObject<UDistributionVectorConstant>(this, TEXT("DistributionLowerLeftCorner"));
	}

	if (!UpperRightCorner.IsCreated())
	{
		UpperRightCorner.Distribution = NewObject<UDistributionVectorConstant>(this, TEXT("DistributionUpperRightCorner"));
	}
}

void UParticleModuleKillBox::PostInitProperties()
{
	Super::PostInitProperties();
	if (!HasAnyFlags(RF_ClassDefaultObject | RF_NeedLoad))
	{
		InitializeDefaults();
	}
}

#if WITH_EDITOR
void UParticleModuleKillBox::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	InitializeDefaults();
	Super::PostEditChangeProperty(PropertyChangedEvent);
}
#endif // WITH_EDITOR

	//## BEGIN PROPS ParticleModuleKillBox
//	struct FRawDistributionVector LowerLeftCorner;
//	struct FRawDistributionVector UpperRightCorner;
//	uint32 bAbsolute:1;
	//## END PROPS ParticleModuleKillBox

void UParticleModuleKillBox::Update(const FUpdateContext& Context)
{
	FParticleEmitterInstance* Owner = &Context.Owner;
	UParticleLODLevel* LODLevel	= Owner->SpriteTemplate->GetCurrentLODLevel(Owner);

	FVector LL = LowerLeftCorner.GetValue(Owner->EmitterTime, Context.GetDistributionData());
	FVector UR = UpperRightCorner.GetValue(Owner->EmitterTime, Context.GetDistributionData());
	FVector CheckLL = LL.ComponentMin(UR);
	FVector CheckUR = UR.ComponentMax(LL);
	if (bAbsolute == false)
	{
		CheckLL += Context.GetTransform().GetLocation();
		CheckUR += Context.GetTransform().GetLocation();
	}
	FBox CheckBox = FBox(CheckLL, CheckUR);

	BEGIN_UPDATE_LOOP;
	{
		FVector Position = Particle.Location + Owner->PositionOffsetThisTick;

		if (LODLevel->RequiredModule->bUseLocalSpace)
		{
			Position = Context.GetTransform().TransformVector(Position);
		}
		else if ((bAxisAlignedAndFixedSize == false) && (bAbsolute == false))
		{
			Position = Context.GetTransform().InverseTransformPosition(Position) + Context.GetTransform().GetLocation();
		}

		// Determine if the particle is inside the box
		bool bIsInside = CheckBox.IsInside(Position);

		// If we are killing on the inside, and it is inside
		//	OR
		// If we are killing on the outside, and it is outside
		// kill the particle
		if (bKillInside == bIsInside)
		{
			// Kill the particle...
			Owner->KillParticle(i);
		}
	}
	END_UPDATE_LOOP;
}

void UParticleModuleKillBox::Render3DPreview(const FPreviewContext& Context)
{
#if WITH_EDITOR
	FParticleEmitterInstance* Owner = &Context.Owner;
	FVector CheckLL = LowerLeftCorner.GetValue(Owner->EmitterTime, Context.GetDistributionData());
	FVector CheckUR = UpperRightCorner.GetValue(Owner->EmitterTime, Context.GetDistributionData());
	
	TArray<FVector> KillboxVerts;
	if (bAxisAlignedAndFixedSize == false)
	{
		KillboxVerts.AddUninitialized(8);
		KillboxVerts[0].Set(CheckLL.X, CheckLL.Y, CheckLL.Z);
		KillboxVerts[1].Set(CheckLL.X, CheckUR.Y, CheckLL.Z);
		KillboxVerts[2].Set(CheckUR.X, CheckUR.Y, CheckLL.Z);
		KillboxVerts[3].Set(CheckUR.X, CheckLL.Y, CheckLL.Z);
		KillboxVerts[4].Set(CheckLL.X, CheckLL.Y, CheckUR.Z);
		KillboxVerts[5].Set(CheckLL.X, CheckUR.Y, CheckUR.Z);
		KillboxVerts[6].Set(CheckUR.X, CheckUR.Y, CheckUR.Z);
		KillboxVerts[7].Set(CheckUR.X, CheckLL.Y, CheckUR.Z);
	}
	
	if (bAbsolute == false)
	{
		if (bAxisAlignedAndFixedSize == false)
		{
			for (int32 i = 0; i < 8; ++i)
			{
				KillboxVerts[i] = Context.GetTransform().TransformPosition(KillboxVerts[i]);
			}
		}
		else
		{
			CheckLL += Context.GetTransform().GetLocation();
			CheckUR += Context.GetTransform().GetLocation();
		}
	}

	FPrimitiveDrawInterface* PDI = Context.PDI;

	if (bAxisAlignedAndFixedSize == false)
	{
		PDI->DrawLine(KillboxVerts[0], KillboxVerts[1], ModuleEditorColor, SDPG_World);
		PDI->DrawLine(KillboxVerts[1], KillboxVerts[2], ModuleEditorColor, SDPG_World);
		PDI->DrawLine(KillboxVerts[2], KillboxVerts[3], ModuleEditorColor, SDPG_World);
		PDI->DrawLine(KillboxVerts[3], KillboxVerts[0], ModuleEditorColor, SDPG_World);
		PDI->DrawLine(KillboxVerts[4], KillboxVerts[5], ModuleEditorColor, SDPG_World);
		PDI->DrawLine(KillboxVerts[5], KillboxVerts[6], ModuleEditorColor, SDPG_World);
		PDI->DrawLine(KillboxVerts[6], KillboxVerts[7], ModuleEditorColor, SDPG_World);
		PDI->DrawLine(KillboxVerts[7], KillboxVerts[4], ModuleEditorColor, SDPG_World);
		PDI->DrawLine(KillboxVerts[0], KillboxVerts[4], ModuleEditorColor, SDPG_World);
		PDI->DrawLine(KillboxVerts[1], KillboxVerts[5], ModuleEditorColor, SDPG_World);
		PDI->DrawLine(KillboxVerts[2], KillboxVerts[6], ModuleEditorColor, SDPG_World);
		PDI->DrawLine(KillboxVerts[3], KillboxVerts[7], ModuleEditorColor, SDPG_World);
	}
	else
	{
		FBox CheckBox = FBox(CheckLL, CheckUR);

		DrawWireBox(PDI, CheckBox, ModuleEditorColor, SDPG_World);
	}
#endif	//#if WITH_EDITOR
}

/*-----------------------------------------------------------------------------
	UParticleModuleKillHeight implementation.
-----------------------------------------------------------------------------*/
UParticleModuleKillHeight::UParticleModuleKillHeight(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bUpdateModule = true;
	bSupported3DDrawMode = true;
}

void UParticleModuleKillHeight::InitializeDefaults()
{
	if (!Height.IsCreated())
	{
		Height.Distribution = NewObject<UDistributionFloatConstant>(this, TEXT("DistributionHeight"));
	}
}

void UParticleModuleKillHeight::PostInitProperties()
{
	Super::PostInitProperties();
	if (!HasAnyFlags(RF_ClassDefaultObject | RF_NeedLoad))
	{
		InitializeDefaults();
	}
}

#if WITH_EDITOR
void UParticleModuleKillHeight::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	InitializeDefaults();
	Super::PostEditChangeProperty(PropertyChangedEvent);
}
#endif // WITH_EDITOR

	//## BEGIN PROPS ParticleModuleKillHeight
//	struct FRawDistributionFloat Height;
//	uint32 bAbsolute:1;
	//## END PROPS ParticleModuleKillHeight
void UParticleModuleKillHeight::Update(const FUpdateContext& Context)
{
	FParticleEmitterInstance* Owner = &Context.Owner;
	UParticleLODLevel* LODLevel	= Owner->SpriteTemplate->GetCurrentLODLevel(Owner);

	float CheckHeight = Height.GetValue(Owner->EmitterTime, Context.GetDistributionData());
	if (bApplyPSysScale == true)
	{
		FVector OwnerScale = Context.GetTransform().GetScale3D();
		CheckHeight *= OwnerScale.Z;
	}

	if (bAbsolute == false)
	{
		CheckHeight += Context.GetTransform().GetLocation().Z;
	}

	BEGIN_UPDATE_LOOP;
	{
		FVector Position = Particle.Location;

		if (LODLevel->RequiredModule->bUseLocalSpace)
		{
			Position = Context.GetTransform().TransformVector(Position);
		}

		if ((bFloor == true) && (Position.Z < CheckHeight))
		{
			// Kill the particle...
			Owner->KillParticle(i);
		}
		else if ((bFloor == false) && (Position.Z > CheckHeight))
		{
			// Kill the particle...
			Owner->KillParticle(i);
		}
	}
	END_UPDATE_LOOP;
}

void UParticleModuleKillHeight::Render3DPreview(const FPreviewContext& Context)
{
#if WITH_EDITOR
	FVector OwnerPosition(0.0f);
	FParticleEmitterInstance* Owner = &Context.Owner;
	{
		OwnerPosition = Context.GetTransform().GetLocation();

		float CheckHeight = Height.GetValue(Owner->EmitterTime, Context.GetDistributionData());
		float ScaleValue = 1.0f;
		if (bApplyPSysScale == true)
		{
			FVector OwnerScale = Context.GetTransform().GetScale3D();
			ScaleValue = OwnerScale.Z;
		}
		CheckHeight *= ScaleValue;
		if (bAbsolute == false)
		{
			CheckHeight += OwnerPosition.Z;
		}

		float Offset = 100.0f * ScaleValue;
		FVector Pt1 = FVector(OwnerPosition.X - Offset, OwnerPosition.Y - Offset, CheckHeight);
		FVector Pt2 = FVector(OwnerPosition.X + Offset, OwnerPosition.Y - Offset, CheckHeight);
		FVector Pt3 = FVector(OwnerPosition.X - Offset, OwnerPosition.Y + Offset, CheckHeight);
		FVector Pt4 = FVector(OwnerPosition.X + Offset, OwnerPosition.Y + Offset, CheckHeight);

		FPrimitiveDrawInterface* PDI = Context.PDI;

		PDI->DrawLine(Pt1, Pt2, ModuleEditorColor, SDPG_World);
		PDI->DrawLine(Pt1, Pt3, ModuleEditorColor, SDPG_World);
		PDI->DrawLine(Pt2, Pt4, ModuleEditorColor, SDPG_World);
		PDI->DrawLine(Pt3, Pt4, ModuleEditorColor, SDPG_World);
	}
#endif	//#if WITH_EDITOR
}




/*-----------------------------------------------------------------------------
	UParticleModuleLifetimeBase implementation.
-----------------------------------------------------------------------------*/
UParticleModuleLifetimeBase::UParticleModuleLifetimeBase(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}
/*-----------------------------------------------------------------------------
	UParticleModuleLifetime implementation.
-----------------------------------------------------------------------------*/

UParticleModuleLifetime::UParticleModuleLifetime(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bSpawnModule = true;
}

void UParticleModuleLifetime::InitializeDefaults()
{
	if(!Lifetime.IsCreated())
	{
		Lifetime.Distribution = NewObject<UDistributionFloatUniform>(this, TEXT("DistributionLifetime"));
	}
}

void UParticleModuleLifetime::PostInitProperties()
{
	Super::PostInitProperties();
	if (!HasAnyFlags(RF_ClassDefaultObject | RF_NeedLoad))
	{
		InitializeDefaults();
	}
}

void UParticleModuleLifetime::CompileModule( struct FParticleEmitterBuildInfo& EmitterInfo )
{
	float MinLifetime;
	float MaxLifetime;

	// Call GetValue once to ensure the distribution has been initialized.
	Lifetime.GetValue();
	Lifetime.GetOutRange( MinLifetime, MaxLifetime );
	EmitterInfo.MaxLifetime = MaxLifetime;
	EmitterInfo.SpawnModules.Add( this );
}

void UParticleModuleLifetime::Spawn(const FSpawnContext& Context)
{
	SpawnEx(Context, &GetRandomStream(Context));
}

void UParticleModuleLifetime::SpawnEx(const FSpawnContext& Context, struct FRandomStream* InRandomStream)
{
	SPAWN_INIT;
	{
		FParticleEmitterInstance* Owner = &Context.Owner;

		float MaxLifetime = Lifetime.GetValue(Owner->EmitterTime, Context.GetDistributionData(), InRandomStream);
		if(Particle.OneOverMaxLifetime > 0.f)
		{
			// Another module already modified lifetime.
			Particle.OneOverMaxLifetime = 1.f / (MaxLifetime + 1.f / Particle.OneOverMaxLifetime);
		}
		else
		{
			// First module to modify lifetime.
			Particle.OneOverMaxLifetime = MaxLifetime > 0.f ? 1.f / MaxLifetime : 0.f;
		}
		//If the relative time is already > 1.0f then we don't want to be setting it. Some modules use this to mark a particle as dead during spawn.
		Particle.RelativeTime = Particle.RelativeTime > 1.0f ? Particle.RelativeTime : Context.SpawnTime * Particle.OneOverMaxLifetime;
	}
}

void UParticleModuleLifetime::SetToSensibleDefaults(UParticleEmitter* Owner)
{
	UDistributionFloatUniform* LifetimeDist = Cast<UDistributionFloatUniform>(Lifetime.Distribution);
	if (LifetimeDist)
	{
		LifetimeDist->Min = 1.0f;
		LifetimeDist->Max = 1.0f;
		LifetimeDist->bIsDirty = true;
	}
}

float UParticleModuleLifetime::GetMaxLifetime()
{
	// Check the distribution for the max value
	float Min, Max;
	Lifetime.GetOutRange(Min, Max);
	return Max;
}

float UParticleModuleLifetime::GetLifetimeValue(const FContext& Context, float InTime, UObject* Data)
{
	return Lifetime.GetValue(InTime, Data);
}

#if WITH_EDITOR
void UParticleModuleLifetime::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	InitializeDefaults();
	Super::PostEditChangeProperty(PropertyChangedEvent);
}
#endif // WITH_EDITOR

/*-----------------------------------------------------------------------------
	UParticleModuleLifetime_Seeded implementation.
-----------------------------------------------------------------------------*/
UParticleModuleLifetime_Seeded::UParticleModuleLifetime_Seeded(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bSpawnModule = true;
	bSupportsRandomSeed = true;
	bRequiresLoopingNotification = true;
}

void UParticleModuleLifetime_Seeded::EmitterLoopingNotify(FParticleEmitterInstance* Owner)
{
	if (RandomSeedInfo.bResetSeedOnEmitterLooping == true)
	{
		FParticleRandomSeedInstancePayload* Payload = Owner->GetModuleRandomSeedInstanceData(this);
		PrepRandomSeedInstancePayload(Owner, Payload, RandomSeedInfo);
	}
}

float UParticleModuleLifetime_Seeded::GetLifetimeValue(const FContext& Context, float InTime, UObject* Data )
{
	return Lifetime.GetValue(InTime, Data, &GetRandomStream(Context));
}

/*-----------------------------------------------------------------------------
	UParticleModuleAttractorBase implementation.
-----------------------------------------------------------------------------*/
UParticleModuleAttractorBase::UParticleModuleAttractorBase(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

/*-----------------------------------------------------------------------------
	UParticleModuleAttractorLine implementation.
-----------------------------------------------------------------------------*/
UParticleModuleAttractorLine::UParticleModuleAttractorLine(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bUpdateModule = true;
	bSupported3DDrawMode = true;
}

void UParticleModuleAttractorLine::InitializeDefaults()
{
	if(!Strength.IsCreated())
	{
		Strength.Distribution = NewObject<UDistributionFloatConstant>(this, TEXT("DistributionStrength"));
	}

	if(!Range.IsCreated())
	{
		Range.Distribution = NewObject<UDistributionFloatConstant>(this, TEXT("DistributionRange"));
	}
}

void UParticleModuleAttractorLine::PostInitProperties()
{
	Super::PostInitProperties();
	if (!HasAnyFlags(RF_ClassDefaultObject | RF_NeedLoad))
	{
		InitializeDefaults();
	}
}

#if WITH_EDITOR
void UParticleModuleAttractorLine::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	InitializeDefaults();
	Super::PostEditChangeProperty(PropertyChangedEvent);
}
#endif // WITH_EDITOR

void UParticleModuleAttractorLine::Update(const FUpdateContext& Context)
{
	FVector Line = EndPoint1 - EndPoint0;

	// if both end points are the same, we end up with NaNs in the results of the update
	if (Line.SizeSquared() == 0.0f)
	{
		Line = FVector(UE_SMALL_NUMBER, UE_SMALL_NUMBER, UE_SMALL_NUMBER);
	}

	FVector LineNorm = Line;
	LineNorm.Normalize();

	FParticleEmitterInstance* Owner = &Context.Owner;

	BEGIN_UPDATE_LOOP;
		// Determine the position of the particle projected on the line
		FVector AdjustedLocation = Particle.Location - Context.GetTransform().GetLocation();
		FVector EP02Particle = AdjustedLocation - EndPoint0;

		FVector ProjectedParticle = Line * (Line | EP02Particle) / (Line | Line);

		// Determine the 'ratio' of the line that has been traveled by the particle
		float VRatioX = 0.0f;
		float VRatioY = 0.0f;
		float VRatioZ = 0.0f;

		if (Line.X)
			VRatioX = (ProjectedParticle.X - EndPoint0.X) / Line.X;
		if (Line.Y)
			VRatioY = (ProjectedParticle.Y - EndPoint0.Y) / Line.Y;
		if (Line.Z)
			VRatioZ = (ProjectedParticle.Z - EndPoint0.Z) / Line.Z;

		bool bProcess = false;
		float fRatio = 0.0f;

		if (VRatioX || VRatioY || VRatioZ)
		{
			// If there are multiple ratios, they should be the same...
			if (VRatioX)
				fRatio = VRatioX;
			else
			if (VRatioY)
				fRatio = VRatioY;
			else
			if (VRatioZ)
				fRatio = VRatioZ;
		}

		if ((fRatio >= 0.0f) && (fRatio <= 1.0f))
			bProcess = true;

		if (bProcess)
		{
			// Look up the Range and Strength at that position on the line
			float AttractorRange = Range.GetValue(fRatio, Context.GetDistributionData());
			FVector LineToPoint = AdjustedLocation - ProjectedParticle;
			float Distance = LineToPoint.Size();

			if ((AttractorRange > 0) && (Distance <= AttractorRange))
			{
				// Adjust the strength based on the range ratio
				float AttractorStrength = Strength.GetValue((AttractorRange - Distance) / AttractorRange, Context.GetDistributionData());
				FVector Direction = LineToPoint^Line;
				// Adjust the VELOCITY of the particle based on the attractor... 
				Particle.Velocity += (FVector3f)Direction * AttractorStrength * DeltaTime;
			}
		}
	END_UPDATE_LOOP;
}

void UParticleModuleAttractorLine::Render3DPreview(const FPreviewContext& Context)
{
#if WITH_EDITOR
	FPrimitiveDrawInterface* PDI = Context.PDI;

	PDI->DrawLine(EndPoint0, EndPoint1, ModuleEditorColor, SDPG_World);

	FParticleEmitterInstance* Owner = &Context.Owner;

	UParticleLODLevel* LODLevel = Owner->SpriteTemplate->GetLODLevel(0);
	check(LODLevel);
	float CurrRatio = Owner->EmitterTime / LODLevel->RequiredModule->EmitterDuration;
	float LineRange = Range.GetValue(CurrRatio, Context.GetDistributionData());

	// Determine the position of the range at this time.
	FVector LinePos = EndPoint0 + CurrRatio * (EndPoint1 - EndPoint0);

	// Draw a wire star at the position of the range.
	DrawWireStar(PDI,LinePos, 10.0f, ModuleEditorColor, SDPG_World);
	// Draw bounding circle for the current range.
	// This should be oriented such that it appears correctly... ie, as 'open' to the camera as possible
	DrawCircle(PDI,LinePos, FVector(1,0,0), FVector(0,1,0), ModuleEditorColor, LineRange, 32, SDPG_World);
#endif	//#if WITH_EDITOR
}

/*-----------------------------------------------------------------------------
	UParticleModuleAttractorParticle implementation.
-----------------------------------------------------------------------------*/
UParticleModuleAttractorParticle::UParticleModuleAttractorParticle(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	// Structure to hold one-time initialization
	struct FConstructorStatics
	{
		FName NAME_None;
		FConstructorStatics()
			: NAME_None(TEXT("None"))
		{
		}
	};
	static FConstructorStatics ConstructorStatics;

	bSpawnModule = true;
	bUpdateModule = true;
	bStrengthByDistance = true;
	bAffectBaseVelocity = false;
	SelectionMethod = EAPSM_Random;
	bRenewSource = false;
	LastSelIndex = 0;
	EmitterName = ConstructorStatics.NAME_None;
}

void UParticleModuleAttractorParticle::InitializeDefaults()
{
	if(!Range.IsCreated())
	{
		Range.Distribution = NewObject<UDistributionFloatConstant>(this, TEXT("DistributionRange"));
	}

	if(!Strength.IsCreated())
	{
		Strength.Distribution = NewObject<UDistributionFloatConstant>(this, TEXT("DistributionStrength"));
	}
}

void UParticleModuleAttractorParticle::PostInitProperties()
{
	Super::PostInitProperties();
	if (!HasAnyFlags(RF_ClassDefaultObject | RF_NeedLoad))
	{
		InitializeDefaults();
	}
}

#if WITH_EDITOR
void UParticleModuleAttractorParticle::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	InitializeDefaults();
	Super::PostEditChangeProperty(PropertyChangedEvent);
}
#endif // WITH_EDITOR

void UParticleModuleAttractorParticle::Spawn(const FSpawnContext& Context)
{
	FParticleEmitterInstance* Owner = &Context.Owner;

	// We need to look up the emitter instance...
	// This may not need to be done every Spawn, but in the short term, it will to be safe.
	// (In the editor, the source emitter may be deleted, etc...)
	FParticleEmitterInstance* AttractorEmitterInst = NULL;
	if (EmitterName != NAME_None)
	{
		TArrayView<FParticleEmitterInstance*> EmitterInstances = Owner->Component.GetEmitterInstances();
		for (int32 ii = 0; ii < EmitterInstances.Num(); ii++)
		{
			FParticleEmitterInstance* pkEmitInst = EmitterInstances[ii];
			if (pkEmitInst && (pkEmitInst->SpriteTemplate->EmitterName == EmitterName))
			{
				AttractorEmitterInst = pkEmitInst;
				break;
			}
		}
	}

	if (AttractorEmitterInst == NULL)
	{
		// No source emitter, so we don't spawn??
		return;
	}

	SPAWN_INIT
	{
		PARTICLE_ELEMENT(FAttractorParticlePayload,	Data);
		
		FBaseParticle* Source = AttractorEmitterInst->GetParticle(LastSelIndex);
		if (Source == NULL)
		{
			switch (SelectionMethod)
			{
			case EAPSM_Random:
				LastSelIndex		= GetRandomStream(Context).RandHelper(AttractorEmitterInst->ActiveParticles);
				Data.SourceIndex	= LastSelIndex;
				break;
			case EAPSM_Sequential:
				{
					for (int32 ui = 0; ui < AttractorEmitterInst->ActiveParticles; ui++)
					{
						Source = AttractorEmitterInst->GetParticle(ui);
						if (Source)
						{
							LastSelIndex		= ui;
							Data.SourceIndex	= LastSelIndex;
							break;
						}
					}
				}
				break;
			}
			Data.SourcePointer	= (uint32)(PTRINT)Source;
			if (Source)
			{
				Data.SourceVelocity	= Source->Velocity;
			}
		}
		else
		{
			Data.SourceIndex = LastSelIndex++;
		}
	}
}


void UParticleModuleAttractorParticle::Update(const FUpdateContext& Context)
{
	FParticleEmitterInstance* Owner = &Context.Owner;

	// We need to look up the emitter instance...
	// This may not need to be done every Spawn, but in the short term, it will to be safe.
	// (In the editor, the source emitter may be deleted, etc...)
	FParticleEmitterInstance* AttractorEmitterInst = NULL;
	if (EmitterName != NAME_None)
	{
		TArrayView<FParticleEmitterInstance*> EmitterInstances = Owner->Component.GetEmitterInstances();
		for (int32 ii = 0; ii < EmitterInstances.Num(); ii++)
		{
			FParticleEmitterInstance* pkEmitInst = EmitterInstances[ii];
			if (pkEmitInst && (pkEmitInst->SpriteTemplate->EmitterName == EmitterName))
			{
				AttractorEmitterInst = pkEmitInst;
				break;
			}
		}
	}

	if (AttractorEmitterInst == NULL)
	{
		// No source emitter, so we don't update??
		return;
	}

	UParticleLODLevel* LODLevel		= Owner->SpriteTemplate->GetCurrentLODLevel(Owner);
	check(LODLevel);
	UParticleLODLevel* SrcLODLevel	= Owner->SpriteTemplate->GetCurrentLODLevel(AttractorEmitterInst);
	check(SrcLODLevel);

	bool bUseLocalSpace = LODLevel->RequiredModule->bUseLocalSpace;
	bool bSrcUseLocalSpace = SrcLODLevel->RequiredModule->bUseLocalSpace;

	BEGIN_UPDATE_LOOP;
	{
		// Find the source particle
		PARTICLE_ELEMENT(FAttractorParticlePayload,	Data);

		if (Data.SourceIndex == 0xffffffff)
		{
#if 1
			if (bInheritSourceVel)
			{
				Particle.Velocity	+= Data.SourceVelocity;
			}
#endif
			CONTINUE_UPDATE_LOOP;
		}

		FBaseParticle* Source = AttractorEmitterInst->GetParticle(Data.SourceIndex);
		if (!Source)
		{
			CONTINUE_UPDATE_LOOP;
		}

		if ((Data.SourcePointer != 0) && 
			(Source != (FBaseParticle*)(PTRINT)(Data.SourcePointer)) && 
			(bRenewSource == false))
		{
			Data.SourceIndex	= 0xffffffff;
			Data.SourcePointer	= 0;
#if 0
			if (bInheritSourceVel)
			{
				Particle.Velocity		+= Data.SourceVelocity;
				Particle.BaseVelocity	+= Data.SourceVelocity;
			}
#endif
			CONTINUE_UPDATE_LOOP;
		}

		float	AttractorRange		= Range.GetValue(Source->RelativeTime, Context.GetDistributionData());
		FVector SrcLocation			= Source->Location;
		FVector	ParticleLocation	= Particle.Location;
		if (bUseLocalSpace != bSrcUseLocalSpace)
		{
			if (bSrcUseLocalSpace)
			{
				SrcLocation = Context.GetTransform().TransformVector(SrcLocation);
			}
			if (bUseLocalSpace)
			{
				ParticleLocation = Context.GetTransform().TransformVector(Particle.Location);
			}
		}

		FVector Dir			= SrcLocation - ParticleLocation;
		float	Distance	= Dir.Size();
		if (Distance <= AttractorRange)
		{
			// Determine the strength
			float AttractorStrength = 0.0f;

			if (bStrengthByDistance)
			{
				// on actual distance
				AttractorStrength = Strength.GetValue((AttractorRange - Distance) / AttractorRange);
			}
			else
			{
				// on emitter time
				AttractorStrength = Strength.GetValue(Source->RelativeTime, Context.GetDistributionData());
			}

			// Adjust the VELOCITY of the particle based on the attractor... 
			Dir.Normalize();
			Particle.Velocity	+= (FVector3f)Dir * AttractorStrength * DeltaTime;
			Data.SourceVelocity	 = Source->Velocity;
			if (bAffectBaseVelocity)
			{
				Particle.BaseVelocity	+= (FVector3f)Dir * AttractorStrength * DeltaTime;
			}
		}
	}
	END_UPDATE_LOOP;
}


uint32 UParticleModuleAttractorParticle::RequiredBytes(UParticleModuleTypeDataBase* TypeData)
{
	return sizeof(FAttractorParticlePayload);
}

/*-----------------------------------------------------------------------------
	UParticleModuleAttractorPoint implementation.
-----------------------------------------------------------------------------*/
UParticleModuleAttractorPoint::UParticleModuleAttractorPoint(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bUpdateModule = true;


	StrengthByDistance = true;
	bAffectBaseVelocity = false;
	bOverrideVelocity = false;
	bSupported3DDrawMode = true;

	Positive_X = true;
	Positive_Y = true;
	Positive_Z = true;

	Negative_X = true;
	Negative_Y = true;
	Negative_Z = true;
}

void UParticleModuleAttractorPoint::InitializeDefaults()
{
	if(!Position.IsCreated())
	{
		Position.Distribution = NewObject<UDistributionVectorConstant>(this, TEXT("DistributionPosition"));
	}
	
	if(!Range.IsCreated())
	{
		Range.Distribution = NewObject<UDistributionFloatConstant>(this, TEXT("DistributionRange"));
	}

	if(!Strength.IsCreated())
	{	
		Strength.Distribution = NewObject<UDistributionFloatConstant>(this, TEXT("DistributionStrength"));
	}
}

void UParticleModuleAttractorPoint::PostInitProperties()
{
	Super::PostInitProperties();
	if (!HasAnyFlags(RF_ClassDefaultObject | RF_NeedLoad))
	{
		InitializeDefaults();
	}
}

#if WITH_EDITOR
void UParticleModuleAttractorPoint::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	InitializeDefaults();
	Super::PostEditChangeProperty(PropertyChangedEvent);
}
#endif // WITH_EDITOR

void UParticleModuleAttractorPoint::Update(const FUpdateContext& Context)
{
	FParticleEmitterInstance* Owner = &Context.Owner;

	// Grab the position of the attractor in Emitter time???
	FVector AttractorPosition = Position.GetValue(Owner->EmitterTime, Context.GetDistributionData());
	float AttractorRange = Range.GetValue(Owner->EmitterTime, Context.GetDistributionData());

	FVector Scale = FVector(1.0f, 1.0f, 1.0f);

	UParticleLODLevel* LODLevel	= Owner->SpriteTemplate->GetCurrentLODLevel(Owner);
	check(LODLevel);
	check(LODLevel->RequiredModule);
	if ( (LODLevel->RequiredModule->bUseLocalSpace == false) && ( bUseWorldSpacePosition == false ) )
	{
		// Transform the attractor into world space
		AttractorPosition = Context.GetTransform().TransformPosition(AttractorPosition);

		Scale *= Context.GetTransform().GetScale3D();
	}
	float ScaleSize = Scale.Size();

	AttractorRange *= ScaleSize;

	FVector MinNormalizedDir(Negative_X ? -1.0f : 0.0f, Negative_Y ? -1.0f : 0.0f, Negative_Z ? -1.0f : 0.0f);
	FVector MaxNormalizedDir(Positive_X ? +1.0f : 0.0f, Positive_Y ? +1.0f : 0.0f, Positive_Z ? +1.0f : 0.0f);

	BEGIN_UPDATE_LOOP;
		// If the particle is within range...
		FVector Dir = AttractorPosition - Particle.Location;
		float Distance = Dir.Size();
		if (Distance <= AttractorRange)
		{
			// Determine the strength
			float AttractorStrength = 0.0f;

			if (StrengthByDistance)
			{
				// on actual distance
				if(AttractorRange == 0.0f)
				{
					AttractorStrength = 0.0f;
				}
				else
				{
					AttractorStrength = Strength.GetValue((AttractorRange - Distance) / AttractorRange, Context.GetDistributionData());
				}
			}
			else
			{
				// on emitter time
				AttractorStrength = Strength.GetValue(Owner->EmitterTime, Context.GetDistributionData());
			}
			if ( (LODLevel->RequiredModule->bUseLocalSpace == false) && ( bUseWorldSpacePosition == false ) )
			{
				AttractorStrength *= ScaleSize;
			}

			Dir.Normalize();

			// If the strength is negative, flip direction before clamping.
			if (AttractorStrength < 0.0f)
			{
				Dir = -Dir;
				AttractorStrength = -AttractorStrength;
			}

			// Adjust the VELOCITY of the particle based on the attractor...
			Dir = ClampVector(Dir,MinNormalizedDir,MaxNormalizedDir);
			Particle.Velocity	+= (FVector3f)Dir * AttractorStrength * DeltaTime;
			if (bAffectBaseVelocity)
			{
				Particle.BaseVelocity	+= (FVector3f)Dir * AttractorStrength * DeltaTime;
			}
		}
	END_UPDATE_LOOP;
}

void UParticleModuleAttractorPoint::Render3DPreview(const FPreviewContext& Context)
{
#if WITH_EDITOR
	FParticleEmitterInstance* Owner = &Context.Owner;
	FVector PointPos = Position.GetValue(Owner->EmitterTime, Context.GetDistributionData());
//    float PointStr = Strength.GetValue(Owner->EmitterTime, Context.GetDistributionData());
	float PointRange = Range.GetValue(Owner->EmitterTime, Context.GetDistributionData());

	FPrimitiveDrawInterface* PDI = Context.PDI;

	// Draw a wire star at the position.
	DrawWireStar(PDI,PointPos, 10.0f, ModuleEditorColor, SDPG_World);
	
	// Draw bounding circles for the range.
	DrawCircle(PDI,PointPos, FVector(1,0,0), FVector(0,1,0), ModuleEditorColor, PointRange, 32, SDPG_World);
	DrawCircle(PDI,PointPos, FVector(1,0,0), FVector(0,0,1), ModuleEditorColor, PointRange, 32, SDPG_World);
	DrawCircle(PDI,PointPos, FVector(0,1,0), FVector(0,0,1), ModuleEditorColor, PointRange, 32, SDPG_World);

	// Draw lines showing the path of travel...
	int32	NumKeys = Position.Distribution->GetNumKeys();
	int32	NumSubCurves = Position.Distribution->GetNumSubCurves();

	FVector InitialPosition;
	FVector SamplePosition[2];

	for (int32 i = 0; i < NumKeys; i++)
	{
		float X = Position.Distribution->GetKeyOut(0, i);
		float Y = Position.Distribution->GetKeyOut(1, i);
		float Z = Position.Distribution->GetKeyOut(2, i);

		if (i == 0)
		{
			InitialPosition.X = X;
			InitialPosition.Y = Y;
			InitialPosition.Z = Z;
			SamplePosition[1].X = X;
			SamplePosition[1].Y = Y;
			SamplePosition[1].Z = Z;
		}
		else
		{
			SamplePosition[0].X = SamplePosition[1].X;
			SamplePosition[0].Y = SamplePosition[1].Y;
			SamplePosition[0].Z = SamplePosition[1].Z;
			SamplePosition[1].X = X;
			SamplePosition[1].Y = Y;
			SamplePosition[1].Z = Z;

			// Draw a line...
			PDI->DrawLine(SamplePosition[0], SamplePosition[1], ModuleEditorColor, SDPG_World);
		}
	}
#endif	//#if WITH_EDITOR
}

/*-----------------------------------------------------------------------------
	Parameter-based distributions
-----------------------------------------------------------------------------*/
UDistributionFloatParticleParameter::UDistributionFloatParticleParameter(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

UDistributionVectorParticleParameter::UDistributionVectorParticleParameter(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

bool UDistributionFloatParticleParameter::GetParamValue(UObject* Data, FName ParamName, float& OutFloat) const
{
	bool bFoundParam = false;

	UParticleSystemComponent* ParticleComp = Cast<UParticleSystemComponent>(Data);
	if(ParticleComp)
	{
		bFoundParam = ParticleComp->GetFloatParameter(ParameterName, OutFloat);
	}

	return bFoundParam;
}

bool UDistributionVectorParticleParameter::GetParamValue(UObject* Data, FName ParamName, FVector& OutVector) const
{
	bool bFoundParam = false;

	UParticleSystemComponent* ParticleComp = Cast<UParticleSystemComponent>(Data);
	if(ParticleComp)
	{
		bFoundParam = ParticleComp->GetAnyVectorParameter(ParameterName, OutVector);
	}
	return bFoundParam;
}

/*-----------------------------------------------------------------------------
	Type data module for GPU particles.
-----------------------------------------------------------------------------*/
UParticleModuleTypeDataGpu::UParticleModuleTypeDataGpu(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, bClearExistingParticlesOnInit(false)
{
}

void UParticleModuleTypeDataGpu::PostLoad()
{
	Super::PostLoad();
	//EmitterInfo.Resources = BeginCreateGPUSpriteResources( ResourceData );

	if (GetLinkerUEVersion() < VER_UE4_OPTIONALLY_CLEAR_GPU_EMITTERS_ON_INIT)
	{
		//Force old emitters to clear their particles on Init() to maintain old behaviour.
		//New emitters are defaulted to false so they behave like other emitter types.
		bClearExistingParticlesOnInit = true;
	}
}

void UParticleModuleTypeDataGpu::BeginDestroy()
{
	if (FGPUSpriteResources* Resources = EmitterInfo.Resources)
	{
		EmitterInfo.Resources = nullptr;
		BeginReleaseGPUSpriteResources(Resources);
	}
	Super::BeginDestroy();
}

void UParticleModuleTypeDataGpu::Build( FParticleEmitterBuildInfo& EmitterBuildInfo )
{
#if WITH_EDITOR
	if (!GetOutermost()->bIsCookedForEditor)
	{
		FVector4Distribution Curve;
		FComposableFloatDistribution ZeroDistribution;
		FComposableFloatDistribution OneDistribution;
		FVectorDistribution VectorDistribution;
		FVector MinValue,MaxValue;
		ZeroDistribution.InitializeWithConstant(0.0f);
		OneDistribution.InitializeWithConstant(1.0f);

		// Store off modules and properties required for simulation.
		EmitterInfo.RequiredModule = EmitterBuildInfo.RequiredModule;
		EmitterInfo.SpawnModule = EmitterBuildInfo.SpawnModule;
		EmitterInfo.SpawnPerUnitModule = EmitterBuildInfo.SpawnPerUnitModule;
		EmitterInfo.SpawnModules = EmitterBuildInfo.SpawnModules;

		// Store the inverse of max size.
		EmitterInfo.InvMaxSize.X = EmitterBuildInfo.MaxSize.X > UE_KINDA_SMALL_NUMBER ? (1.0f / EmitterBuildInfo.MaxSize.X) : 1.0f;
		EmitterInfo.InvMaxSize.Y = EmitterBuildInfo.MaxSize.Y > UE_KINDA_SMALL_NUMBER ? (1.0f / EmitterBuildInfo.MaxSize.Y) : 1.0f;

		// Compute the value by which to scale rotation rate.
		const float RotationRateScale = EmitterBuildInfo.MaxRotationRate * EmitterBuildInfo.MaxLifetime;

		// Store the maximum rotation rate (make sure it is never zero).
		EmitterInfo.InvRotationRateScale = (RotationRateScale > UE_KINDA_SMALL_NUMBER || RotationRateScale < -UE_KINDA_SMALL_NUMBER) ?
			1.0f / RotationRateScale : 1.0f;

		// A particle's initial size is stored as 1 / MaxSize, so scale by MaxSize.
		EmitterBuildInfo.SizeScale.ScaleByConstantVector( FVector( EmitterBuildInfo.MaxSize.X, EmitterBuildInfo.MaxSize.Y, 0.0f ) );

		// Build and store the color curve.
		EmitterBuildInfo.ColorScale.Resample(0.0f, 1.0f);
		EmitterBuildInfo.AlphaScale.Resample(0.0f, 1.0f);
		FComposableDistribution::BuildVector4(
			Curve,
			EmitterBuildInfo.ColorScale,
			EmitterBuildInfo.AlphaScale );
		FComposableDistribution::QuantizeVector4(
			ResourceData.QuantizedColorSamples,
			ResourceData.ColorScale,
			ResourceData.ColorBias,
			Curve );

		// The misc curve is laid out as: R:SizeX G:SizeY B:SubImageIndex A:Unused.
		EmitterBuildInfo.SizeScale.Resample(0.0f, 1.0f);
		EmitterBuildInfo.SubImageIndex.Resample(0.0f, 1.0f);
		FComposableDistribution::BuildVector4(
			Curve, 
			EmitterBuildInfo.SizeScale, 
			EmitterBuildInfo.SubImageIndex, 
			ZeroDistribution );
		FComposableDistribution::QuantizeVector4(
			ResourceData.QuantizedMiscSamples,
			ResourceData.MiscScale,
			ResourceData.MiscBias,
			Curve );

		// Resilience.
		bool bBounceOnCollision = EmitterBuildInfo.CollisionResponse == EParticleCollisionResponse::Bounce;
		FComposableFloatDistribution NormalizedResilience(
			bBounceOnCollision ? EmitterBuildInfo.Resilience : ZeroDistribution
			);
		NormalizedResilience.Normalize(&ResourceData.ResilienceScale, &ResourceData.ResilienceBias);
		FComposableDistribution::BuildFloat(EmitterInfo.Resilience, NormalizedResilience);

		// The simulation attributes curve is: R:DragScale G:VelocityFieldScale B:Resilience A:OrbitRandom.
		EmitterBuildInfo.VectorFieldScaleOverLife.Resample(0.0f, 1.0f);
		EmitterBuildInfo.DragScale.Resample(0.0f, 1.0f);
		EmitterBuildInfo.ResilienceScaleOverLife.Resample(0.0f, 1.0f);
		FComposableDistribution::BuildVector4(
			Curve,
			EmitterBuildInfo.DragScale,
			EmitterBuildInfo.VectorFieldScaleOverLife,
			EmitterBuildInfo.ResilienceScaleOverLife,
			OneDistribution );
		FComposableDistribution::QuantizeVector4(
			ResourceData.QuantizedSimulationAttrSamples,
			ResourceData.SimulationAttrCurveScale,
			ResourceData.SimulationAttrCurveBias,
			Curve );

		// Friction used during collision.
		if (bBounceOnCollision)
		{
			ResourceData.OneMinusFriction = 1.0f - EmitterBuildInfo.Friction;
			ResourceData.CollisionRandomSpread = EmitterBuildInfo.CollisionRandomSpread;
			ResourceData.CollisionRandomDistribution = EmitterBuildInfo.CollisionRandomDistribution;
		}
		else
		{
			ResourceData.OneMinusFriction = 0.0f;
			ResourceData.CollisionRandomSpread = 0.0f;
			ResourceData.CollisionRandomDistribution = 1.0f;
		}

		// Collision time bias, used to kill particles on collision if desired.
		if (EmitterBuildInfo.CollisionResponse == EParticleCollisionResponse::Kill)
		{
			// By adding 1.1 to relative time it will kill the particle.
			ResourceData.CollisionTimeBias = 1.1f;
		}
		else
		{
			ResourceData.CollisionTimeBias = 0.0f;
		}

		// Parameters used to derive the collision radius from the size of the sprite.
		// Note that the sprite size is the diameter, so bake a 1/2 in to the radius
		// scale to convert to radius.
		ResourceData.CollisionRadiusScale = EmitterBuildInfo.CollisionRadiusScale * 0.5f;
		ResourceData.CollisionRadiusBias = EmitterBuildInfo.CollisionRadiusBias;

		// If appropriate, set up the sub-image size parameter.
		EParticleSubUVInterpMethod InterpMethod = EmitterBuildInfo.RequiredModule->InterpolationMethod;
		if ( InterpMethod == PSUVIM_Linear || InterpMethod == PSUVIM_Linear_Blend )
		{
			ResourceData.SubImageSize.X = EmitterBuildInfo.RequiredModule->SubImages_Horizontal;
			ResourceData.SubImageSize.Y = EmitterBuildInfo.RequiredModule->SubImages_Vertical;
			ResourceData.SubImageSize.Z = 1.0f / ResourceData.SubImageSize.X;
			ResourceData.SubImageSize.W = 1.0f / ResourceData.SubImageSize.Y;
		}
		else
		{
			ResourceData.SubImageSize = FVector4( 1.0f, 1.0f, 1.0f, 1.0f );
		}

		// Store the size-by-speed parameters.
		ResourceData.SizeBySpeed.X = FMath::Max<float>(EmitterBuildInfo.SizeScaleBySpeed.X, 0.0f);
		ResourceData.SizeBySpeed.Y = FMath::Max<float>(EmitterBuildInfo.SizeScaleBySpeed.Y, 0.0f);
		ResourceData.SizeBySpeed.Z = FMath::Max<float>(EmitterBuildInfo.MaxSizeScaleBySpeed.X, 0.0f);
		ResourceData.SizeBySpeed.W = FMath::Max<float>(EmitterBuildInfo.MaxSizeScaleBySpeed.Y, 0.0f);

		// Point attractor.
		{
			float RadiusSq = EmitterBuildInfo.PointAttractorRadius *
				EmitterBuildInfo.PointAttractorRadius;
			EmitterBuildInfo.PointAttractorStrength.ScaleByConstant(RadiusSq);
			FComposableDistribution::BuildFloat(
				EmitterInfo.PointAttractorStrength,
				EmitterBuildInfo.PointAttractorStrength );
			EmitterInfo.PointAttractorPosition = EmitterBuildInfo.PointAttractorPosition;
			EmitterInfo.PointAttractorRadiusSq = RadiusSq;
		}

		// Store the constant acceleration to apply to particles.
		ResourceData.ConstantAcceleration = EmitterBuildInfo.ConstantAcceleration;
		EmitterInfo.ConstantAcceleration = EmitterBuildInfo.ConstantAcceleration;

		// Compute the orbit offset amount.
		FComposableDistribution::BuildVector(VectorDistribution, EmitterBuildInfo.OrbitOffset);
		VectorDistribution.GetRange(&MinValue, &MaxValue);

		// One half required due to integration in the shader.
		MinValue *= 0.5f;
		MaxValue *= 0.5f;

		// Store the orbit offset range.
		ResourceData.OrbitOffsetBase = MinValue;
		ResourceData.OrbitOffsetRange = MaxValue - MinValue;

		// Compute the orbit frequencies.
		FComposableDistribution::BuildVector(VectorDistribution, EmitterBuildInfo.OrbitRotationRate);
		VectorDistribution.GetRange(&MinValue, &MaxValue);

		// # rotations to radians. Flip Z to be consistent with CPU orbit.
		MinValue *= (2.0f * UE_PI);
		MaxValue *= (2.0f * UE_PI);
		MinValue.Z *= -1.0f;
		MaxValue.Z *= -1.0f;

		// Store the orbit frequency range.
		ResourceData.OrbitFrequencyBase = MinValue;
		ResourceData.OrbitFrequencyRange = MaxValue - MinValue;

		// Compute the orbit phase.
		FComposableDistribution::BuildVector(VectorDistribution, EmitterBuildInfo.OrbitInitialRotation);
		VectorDistribution.GetRange(&MinValue, &MaxValue);

		// # rotations to radians. Flip Z to be consistent with CPU orbit.
		MinValue *= (2.0f * UE_PI);
		MaxValue *= (2.0f * UE_PI);
		MinValue.Z *= -1.0f;
		MaxValue.Z *= -1.0f;

		// Store the orbit phase range.
		ResourceData.OrbitPhaseBase = MinValue;
		ResourceData.OrbitPhaseRange = MaxValue - MinValue;

		// Determine around which axes particles are orbiting.
		const float OrbitX = (ResourceData.OrbitFrequencyBase.X != 0.0f || ResourceData.OrbitFrequencyRange.X != 0.0f || ResourceData.OrbitPhaseBase.X != 0.0f || ResourceData.OrbitPhaseRange.X != 0.0f) ? 1.0f : 0.0f;
		const float OrbitY = (ResourceData.OrbitFrequencyBase.Y != 0.0f || ResourceData.OrbitFrequencyRange.Y != 0.0f || ResourceData.OrbitPhaseBase.Y != 0.0f || ResourceData.OrbitPhaseRange.Y != 0.0f) ? 1.0f : 0.0f;
		const float OrbitZ = (ResourceData.OrbitFrequencyBase.Z != 0.0f || ResourceData.OrbitFrequencyRange.Z != 0.0f || ResourceData.OrbitPhaseBase.Z != 0.0f || ResourceData.OrbitPhaseRange.Z != 0.0f) ? 1.0f : 0.0f;

		// Make some adjustments to mimic CPU orbit as much as possible.
		if (OrbitX != 0.0f)
		{
			ResourceData.OrbitPhaseBase.X += (0.5f * UE_PI);
		}

		if (OrbitZ != 0.0f)
		{
			ResourceData.OrbitPhaseBase.Z += (0.5f * UE_PI);
		}

		// Compute an offset to position the particle at the beginning of its orbit.
		EmitterInfo.OrbitOffsetBase.X = 2.0f * ResourceData.OrbitOffsetBase.X * (OrbitY * FMath::Cos(ResourceData.OrbitPhaseBase.Y) + OrbitZ * FMath::Sin(ResourceData.OrbitPhaseBase.Z));
		EmitterInfo.OrbitOffsetBase.Y = 2.0f * ResourceData.OrbitOffsetBase.Y * (OrbitZ * FMath::Cos(ResourceData.OrbitPhaseBase.Z) + OrbitX * FMath::Sin(ResourceData.OrbitPhaseBase.X));
		EmitterInfo.OrbitOffsetBase.Z = 2.0f * ResourceData.OrbitOffsetBase.Z * (OrbitX * FMath::Cos(ResourceData.OrbitPhaseBase.X) + OrbitY * FMath::Sin(ResourceData.OrbitPhaseBase.Y));
		EmitterInfo.OrbitOffsetRange.X = -EmitterInfo.OrbitOffsetBase.X + 2.0f * (ResourceData.OrbitOffsetBase.X + ResourceData.OrbitOffsetRange.X)
			* (OrbitY * FMath::Cos(ResourceData.OrbitPhaseBase.Y + ResourceData.OrbitPhaseRange.Y) + OrbitZ * FMath::Sin(ResourceData.OrbitPhaseBase.Z + ResourceData.OrbitPhaseRange.Z));
		EmitterInfo.OrbitOffsetRange.Y = -EmitterInfo.OrbitOffsetBase.Y + 2.0f * (ResourceData.OrbitOffsetBase.Y + ResourceData.OrbitOffsetRange.Y)
			* (OrbitZ * FMath::Cos(ResourceData.OrbitPhaseBase.Z + ResourceData.OrbitPhaseRange.Z) + OrbitX * FMath::Sin(ResourceData.OrbitPhaseBase.X + ResourceData.OrbitPhaseRange.X));
		EmitterInfo.OrbitOffsetRange.Z = -EmitterInfo.OrbitOffsetBase.Z + 2.0f * (ResourceData.OrbitOffsetBase.Z + ResourceData.OrbitOffsetRange.Z)
			* (OrbitX * FMath::Cos(ResourceData.OrbitPhaseBase.X + ResourceData.OrbitPhaseRange.X) + OrbitY * FMath::Sin(ResourceData.OrbitPhaseBase.Y + ResourceData.OrbitPhaseRange.Y));

		// Local vector field.
		EmitterInfo.LocalVectorField.Field = EmitterBuildInfo.LocalVectorField;
		EmitterInfo.LocalVectorField.Transform = EmitterBuildInfo.LocalVectorFieldTransform;
		EmitterInfo.LocalVectorField.MinInitialRotation = FRotator::MakeFromEuler(
			EmitterBuildInfo.LocalVectorFieldMinInitialRotation * 360.0f );
		EmitterInfo.LocalVectorField.MaxInitialRotation = FRotator::MakeFromEuler(
			EmitterBuildInfo.LocalVectorFieldMaxInitialRotation * 360.0f );
		EmitterInfo.LocalVectorField.RotationRate = FRotator::MakeFromEuler(
			EmitterBuildInfo.LocalVectorFieldRotationRate * 360.0f );
		EmitterInfo.LocalVectorField.Intensity = EmitterBuildInfo.LocalVectorFieldIntensity;
		EmitterInfo.LocalVectorField.Tightness = EmitterBuildInfo.LocalVectorFieldTightness;
		EmitterInfo.LocalVectorField.bIgnoreComponentTransform = EmitterBuildInfo.bLocalVectorFieldIgnoreComponentTransform;
		EmitterInfo.LocalVectorField.bTileX = EmitterBuildInfo.bLocalVectorFieldTileX;
		EmitterInfo.LocalVectorField.bTileY = EmitterBuildInfo.bLocalVectorFieldTileY;
		EmitterInfo.LocalVectorField.bTileZ = EmitterBuildInfo.bLocalVectorFieldTileZ;
		EmitterInfo.LocalVectorField.bUseFixDT = EmitterBuildInfo.bLocalVectorFieldUseFixDT;

		// Vector field scales.
		FComposableFloatDistribution NormalizedVectorFieldScale(EmitterBuildInfo.VectorFieldScale);
		NormalizedVectorFieldScale.Normalize(&ResourceData.PerParticleVectorFieldScale, &ResourceData.PerParticleVectorFieldBias);
		FComposableDistribution::BuildFloat(EmitterInfo.VectorFieldScale, NormalizedVectorFieldScale);

		if (EmitterBuildInfo.RequiredModule->bUseLocalSpace)
		{
			ResourceData.GlobalVectorFieldScale = 0.0f;
			ResourceData.GlobalVectorFieldTightness = -1;
		}
		else
		{
			ResourceData.GlobalVectorFieldScale = EmitterBuildInfo.GlobalVectorFieldScale;
			ResourceData.GlobalVectorFieldTightness = EmitterBuildInfo.GlobalVectorFieldTightness;
		}

		// Drag coefficient.
		FComposableFloatDistribution NormalizedDragCoefficient(EmitterBuildInfo.DragCoefficient);
		NormalizedDragCoefficient.Normalize(&ResourceData.DragCoefficientScale, &ResourceData.DragCoefficientBias);
		FComposableDistribution::BuildFloat(EmitterInfo.DragCoefficient, NormalizedDragCoefficient);

		// Set the scale by which rotation rate must be multiplied.
		ResourceData.RotationRateScale = RotationRateScale;

		// Camera motion blur.
		ResourceData.CameraMotionBlurAmount = this->CameraMotionBlurAmount;

		// Compute the maximum lifetime of particles in this emitter. */
		EmitterInfo.MaxLifetime = 0.0f;
		for (int32 ModuleIndex = 0; ModuleIndex < EmitterInfo.SpawnModules.Num(); ++ModuleIndex)
		{
			UParticleModuleLifetimeBase* LifetimeModule = Cast<UParticleModuleLifetimeBase>(EmitterInfo.SpawnModules[ModuleIndex]);
			if (LifetimeModule)
			{
				EmitterInfo.MaxLifetime += LifetimeModule->GetMaxLifetime();
			}
		}

		// Compute the maximum number of particles allowed for this emitter.
		EmitterInfo.MaxParticleCount = FMath::Max<int32>( 1, EmitterBuildInfo.EstimatedMaxActiveParticleCount );

		EmitterInfo.bUseVelocityForMotionBlur = EmitterBuildInfo.RequiredModule->ShouldUseVelocityForMotionBlur();
		ResourceData.bUseVelocityForMotionBlur = EmitterBuildInfo.RequiredModule->ShouldUseVelocityForMotionBlur();

		// Store screen alignment for particles.
		EmitterInfo.ScreenAlignment = EmitterBuildInfo.RequiredModule->ScreenAlignment;
		ResourceData.ScreenAlignment = EmitterBuildInfo.RequiredModule->ScreenAlignment;

		EmitterInfo.bRemoveHMDRoll = EmitterBuildInfo.RequiredModule->bRemoveHMDRoll;
		EmitterInfo.MinFacingCameraBlendDistance = EmitterBuildInfo.RequiredModule->MinFacingCameraBlendDistance;
		EmitterInfo.MaxFacingCameraBlendDistance = EmitterBuildInfo.RequiredModule->MaxFacingCameraBlendDistance;
	
		ResourceData.bRemoveHMDRoll = EmitterBuildInfo.RequiredModule->bRemoveHMDRoll;
		ResourceData.MinFacingCameraBlendDistance = EmitterBuildInfo.RequiredModule->MinFacingCameraBlendDistance;
		ResourceData.MaxFacingCameraBlendDistance = EmitterBuildInfo.RequiredModule->MaxFacingCameraBlendDistance;

		// Particle axis lock
		for (int32 ModuleIndex = 0; ModuleIndex < EmitterInfo.SpawnModules.Num(); ++ModuleIndex)
		{
			UParticleModuleOrientationAxisLock* AxisLockModule = Cast<UParticleModuleOrientationAxisLock>(EmitterInfo.SpawnModules[ModuleIndex]);
			if (AxisLockModule)
			{
				EmitterInfo.LockAxisFlag = AxisLockModule->LockAxisFlags;
				ResourceData.LockAxisFlag = AxisLockModule->LockAxisFlags;
				break;
			}
		}

		ResourceData.PivotOffset = EmitterBuildInfo.PivotOffset;

		// Store color and scale when using particle parameters.
		EmitterInfo.DynamicColor = EmitterBuildInfo.DynamicColor;
		EmitterInfo.DynamicAlpha= EmitterBuildInfo.DynamicAlpha;
		EmitterInfo.DynamicColorScale = EmitterBuildInfo.DynamicColorScale;
		EmitterInfo.DynamicAlphaScale = EmitterBuildInfo.DynamicAlphaScale;

		// Collision flag.
		EmitterInfo.bEnableCollision = EmitterBuildInfo.bEnableCollision;
		EmitterInfo.CollisionMode = (EParticleCollisionMode::Type)EmitterBuildInfo.CollisionMode;
	}
#endif

	// Create or update GPU resources.
	if ( EmitterInfo.Resources )
	{
		BeginUpdateGPUSpriteResources( EmitterInfo.Resources, ResourceData );
	}
	else
	{
		EmitterInfo.Resources = BeginCreateGPUSpriteResources( ResourceData );
	}
}

FParticleEmitterInstance* UParticleModuleTypeDataGpu::CreateInstance(UParticleEmitter* InEmitterParent, IParticleEmitterInstanceOwner& InComponent)
{
	UE_LOGF(LogParticles,Verbose,
		"Create GPU Sprite Emitter @ %fs %ls", InComponent.GetWorldTimeSeconds(),
		InComponent.GetTemplate() != NULL ? *InComponent.GetTemplate()->GetName() : TEXT("NULL"));

	FParticleEmitterInstance* Instance = NULL;
	if (InComponent.GetScene() && RHISupportsGPUParticles())
	{
		LLM_SCOPE(ELLMTag::Particles);

		check(InComponent.GetFXSystem());
		Instance = InComponent.GetFXSystem()->CreateGPUSpriteEmitterInstance(InComponent, EmitterInfo );
		Instance->InitParameters( InEmitterParent );
	}
	return Instance;
}

const FVertexFactoryType* UParticleModuleTypeDataGpu::GetVertexFactoryType() const
{
	return &FGPUSpriteVertexFactory::StaticType;
}

void UParticleModuleTypeDataGpu::CollectPSOPrecacheData(const UParticleEmitter* Emitter, FPSOPrecacheParams& OutParams)
{
	OutParams.VertexFactoryDataList.Add(FPSOPrecacheVertexFactoryData(GetVertexFactoryType()));
	OutParams.PrimitiveType = GetPrimitiveType();
}

/*-----------------------------------------------------------------------------
	UParticleModulePivotOffset implementation.
-----------------------------------------------------------------------------*/

UParticleModulePivotOffset::UParticleModulePivotOffset(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bSpawnModule = false;
	bUpdateModule = false;
}

void UParticleModulePivotOffset::InitializeDefaults()
{
	 PivotOffset = FVector2D(0.0f,0.0f);
}

void UParticleModulePivotOffset::PostInitProperties()
{
	Super::PostInitProperties();
	if (!HasAnyFlags(RF_ClassDefaultObject | RF_NeedLoad))
	{
		InitializeDefaults();
	}
}

void UParticleModulePivotOffset::CompileModule(FParticleEmitterBuildInfo& EmitterInfo)
{
	EmitterInfo.PivotOffset = PivotOffset - FVector2D(0.5f,0.5f);
}

#if WITH_EDITOR
bool UParticleModulePivotOffset::IsValidForLODLevel(UParticleLODLevel* LODLevel, FString& OutErrorString)
{
	if ( !LODLevel->TypeDataModule || LODLevel->TypeDataModule->IsA(UParticleModuleTypeDataGpu::StaticClass()) ) 
	{
		return true;
	}

	return false;
}
#endif // WITH_EDITOR
