// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	ParticleModules_Camera.cpp: 
	Camera-related particle module implementations.
=============================================================================*/

#include "ParticleEmitterInstances.h"
#include "ParticleEmitterInstanceOwner.h"
#include "Particles/ParticleSystemComponent.h"
#include "Distributions/DistributionFloatConstant.h"
#include "Particles/Camera/ParticleModuleCameraBase.h"
#include "Particles/Camera/ParticleModuleCameraOffset.h"
#include "Particles/ParticleEmitter.h"
#include "Particles/ParticleLODLevel.h"
#include "Particles/ParticleModule.h"
#include "Particles/ParticleModuleRequired.h"

UParticleModuleCameraBase::UParticleModuleCameraBase(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

/*-----------------------------------------------------------------------------
	Abstract base modules used for categorization.
-----------------------------------------------------------------------------*/

/*-----------------------------------------------------------------------------
	UParticleModuleCameraOffset
-----------------------------------------------------------------------------*/
UParticleModuleCameraOffset::UParticleModuleCameraOffset(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bSpawnModule = true;
	bUpdateModule = true;
	bSpawnTimeOnly = false;
	UpdateMethod = EPCOUM_DirectSet;
}

void UParticleModuleCameraOffset::InitializeDefaults()
{
	if (!CameraOffset.IsCreated())
	{
		UDistributionFloatConstant* DistributionCameraOffset = NewObject<UDistributionFloatConstant>(this, TEXT("DistributionCameraOffset"));
		DistributionCameraOffset->Constant = 1.0f;
		CameraOffset.Distribution = DistributionCameraOffset; 
	}
}

void UParticleModuleCameraOffset::PostInitProperties()
{
	Super::PostInitProperties();
	if (!HasAnyFlags(RF_ClassDefaultObject | RF_NeedLoad))
	{
		InitializeDefaults();
	}
}

#if WITH_EDITOR
void UParticleModuleCameraOffset::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	InitializeDefaults();
	Super::PostEditChangeProperty(PropertyChangedEvent);
}
#endif // WITH_EDITOR

bool UParticleModuleCameraOffset::CanTickInAnyThread()
{
	return CameraOffset.OkForParallel();
}


void UParticleModuleCameraOffset::Spawn(const FSpawnContext& Context)
{
	FParticleEmitterInstance* Owner = &Context.Owner;

	float ScaleFactor = 1.0f;

	if (Owner)
	{
		UParticleLODLevel* LODLevel = Owner->SpriteTemplate->GetCurrentLODLevel(Owner);
		if (LODLevel && LODLevel->RequiredModule->bUseLocalSpace == false)
		{
			ScaleFactor = Owner->Component.GetAsyncComponentToWorld().GetMaximumAxisScale();
		}
	}
	SPAWN_INIT;
	{
		CurrentOffset = Owner ? ((Owner->CameraPayloadOffset != 0) ? Owner->CameraPayloadOffset : Context.Offset) : Context.Offset;
		PARTICLE_ELEMENT(FCameraOffsetParticlePayload, CameraPayload);
		float CameraOffsetValue = CameraOffset.GetValue(Particle.RelativeTime, Context.GetDistributionData()) * ScaleFactor;
		if (UpdateMethod == EPCOUM_DirectSet)
		{
			CameraPayload.BaseOffset = CameraOffsetValue;
			CameraPayload.Offset = CameraOffsetValue;
		}
		else if (UpdateMethod == EPCOUM_Additive)
		{
			CameraPayload.Offset += CameraOffsetValue;
		}
		else //if (UpdateMethod == EPCOUM_Scalar)
		{
			CameraPayload.Offset *= CameraOffsetValue;
		}
	}
}

void UParticleModuleCameraOffset::Update(const FUpdateContext& Context)
{
	if (bSpawnTimeOnly == false)
	{
		FParticleEmitterInstance* Owner = &Context.Owner;																	\
		BEGIN_UPDATE_LOOP;
		{
			CurrentOffset = Owner ? ((Owner->CameraPayloadOffset != 0) ? Owner->CameraPayloadOffset : Offset) : Offset;
			PARTICLE_ELEMENT(FCameraOffsetParticlePayload, CameraPayload);
			float CameraOffsetValue = CameraOffset.GetValue(Particle.RelativeTime, Context.GetDistributionData());
			if (UpdateMethod == EPCOUM_Additive)
			{
				CameraPayload.Offset += CameraOffsetValue;
			}
			else if (UpdateMethod == EPCOUM_Scalar)
			{
				CameraPayload.Offset *= CameraOffsetValue;
			}
			else //if (UpdateMethod == EPCOUM_DirectSet)
			{
				CameraPayload.Offset = CameraOffsetValue;
			}
		}
		END_UPDATE_LOOP;
	}
}

uint32 UParticleModuleCameraOffset::RequiredBytes(UParticleModuleTypeDataBase* TypeData)
{
	return sizeof(FCameraOffsetParticlePayload);
}
