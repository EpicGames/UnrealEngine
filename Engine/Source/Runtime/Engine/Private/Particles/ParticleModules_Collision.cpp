// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	ParticleModules_Collision.cpp: 
	Collision-related particle module implementations.
=============================================================================*/

#include "Components/PrimitiveComponent.h"
#include "Engine/World.h"
#include "GameFramework/Pawn.h"
#include "MaterialDomain.h"
#include "Materials/Material.h"
#include "MaterialShared.h"
#include "Distributions/DistributionFloatConstant.h"
#include "Distributions/DistributionFloatUniform.h"
#include "Distributions/DistributionVectorConstant.h"
#include "Distributions/DistributionVectorUniform.h"
#include "Engine/StaticMesh.h"
#include "Engine/TriggerBase.h"
#include "ParticleEmitterInstances.h"
#include "ParticleEmitterInstanceOwner.h"
#include "Particles/Collision/ParticleModuleCollisionBase.h"
#include "Particles/Collision/ParticleModuleCollision.h"
#include "Particles/Collision/ParticleModuleCollisionGPU.h"
#include "Particles/Event/ParticleModuleEventGenerator.h"
#include "Particles/ParticleEmitter.h"
#include "Particles/TypeData/ParticleModuleTypeDataMesh.h"
#include "Particles/ParticleModule.h"
#include "Particles/TypeData/ParticleModuleTypeDataGpu.h"
#include "Particles/ParticleLODLevel.h"
#include "Particles/ParticleModuleRequired.h"
#include "Particles/ParticleSystem.h"
#include "Particles/ParticleSystemComponent.h"
#include "Particles/TypeData/ParticleModuleTypeDataBase.h"
#include "SceneManagement.h"

UParticleModuleCollisionBase::UParticleModuleCollisionBase(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

/*-----------------------------------------------------------------------------
	FParticlesStatGroup
-----------------------------------------------------------------------------*/
DEFINE_STAT(STAT_ParticleCollisionTime);

/*-----------------------------------------------------------------------------
	Abstract base modules used for categorization.
-----------------------------------------------------------------------------*/

/*-----------------------------------------------------------------------------
	UParticleModuleCollision implementation.
-----------------------------------------------------------------------------*/
UParticleModuleCollision::UParticleModuleCollision(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bSpawnModule = true;
	bUpdateModule = true;
	CollisionCompletionOption = EPCC_Kill;
	bApplyPhysics = false;
	DirScalar = 3.5f;
	VerticalFudgeFactor = 0.1f;
	LODDuplicate = false;
	bPawnsDoNotDecrementCount = true;
	bCollideOnlyIfVisible = true;
	MaxCollisionDistance = 1000.0f;
	bIgnoreSourceActor = true;
	bIgnoreTriggerVolumes = true;
	CollisionTypes.Add(UEngineTypes::ConvertToObjectType(ECC_WorldStatic));
}

void UParticleModuleCollision::InitializeDefaults()
{
	if (!DampingFactor.IsCreated())
	{
		DampingFactor.Distribution = NewObject<UDistributionVectorUniform>(this, TEXT("DistributionDampingFactor"));
	}

	if (!DampingFactorRotation.IsCreated())
	{
		UDistributionVectorConstant* DistributionDampingFactorRotation = NewObject<UDistributionVectorConstant>(this, TEXT("DistributionDampingFactorRotation"));
		DistributionDampingFactorRotation->Constant = FVector(1.0f, 1.0f, 1.0f);
		DampingFactorRotation.Distribution = DistributionDampingFactorRotation; 
	}

	if (!MaxCollisions.IsCreated())
	{
		MaxCollisions.Distribution = NewObject<UDistributionFloatUniform>(this, TEXT("DistributionMaxCollisions"));
	}

	if (!ParticleMass.IsCreated())
	{
		UDistributionFloatConstant* DistributionParticleMass = NewObject<UDistributionFloatConstant>(this, TEXT("DistributionParticleMass"));
		DistributionParticleMass->Constant = 0.1f;
		ParticleMass.Distribution = DistributionParticleMass;
	}

	if (!DelayAmount.IsCreated())
	{
		UDistributionFloatConstant* DistributionDelayAmount = NewObject<UDistributionFloatConstant>(this, TEXT("DistributionDelayAmount"));
		DistributionDelayAmount->Constant = 0.0f;
		DelayAmount.Distribution = DistributionDelayAmount;
	}

	ObjectParams = FCollisionObjectQueryParams(CollisionTypes);
}

void UParticleModuleCollision::PostInitProperties()
{
	Super::PostInitProperties();
	if (!HasAnyFlags(RF_ClassDefaultObject | RF_NeedLoad))
	{
		InitializeDefaults();
	}
}

void UParticleModuleCollision::PostLoad()
{
	Super::PostLoad();

	ObjectParams = FCollisionObjectQueryParams(CollisionTypes);
}

#if WITH_EDITOR
void UParticleModuleCollision::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	InitializeDefaults();
	Super::PostEditChangeProperty(PropertyChangedEvent);
}
#endif // WITH_EDITOR

uint32 UParticleModuleCollision::RequiredBytes(UParticleModuleTypeDataBase* TypeData)
{
	return sizeof(FParticleCollisionPayload);
}


uint32 UParticleModuleCollision::RequiredBytesPerInstance()
{
	return sizeof(FParticleCollisionInstancePayload);
}


uint32 UParticleModuleCollision::PrepPerInstanceBlock(FParticleEmitterInstance* Owner, void* InstData)
{
	FParticleCollisionInstancePayload* CollisionInstPayload = (FParticleCollisionInstancePayload*)(InstData);
	CollisionInstPayload->CurrentLODBoundsCheckCount = 0;
	return 0;
}

void UParticleModuleCollision::Spawn(const FSpawnContext& Context)
{
	SCOPE_CYCLE_COUNTER(STAT_ParticleCollisionTime);

	FParticleEmitterInstance* Owner = &Context.Owner;

	SPAWN_INIT;
	{
		PARTICLE_ELEMENT(FParticleCollisionPayload, CollisionPayload);
		CollisionPayload.UsedDampingFactor = (FVector3f)DampingFactor.GetValue(Owner->EmitterTime, Context.GetDistributionData());
		CollisionPayload.UsedDampingFactorRotation = (FVector3f)DampingFactorRotation.GetValue(Owner->EmitterTime, Context.GetDistributionData());
		CollisionPayload.UsedCollisions = FMath::RoundToInt(MaxCollisions.GetValue(Owner->EmitterTime, Context.GetDistributionData()));
		CollisionPayload.Delay = DelayAmount.GetValue(Owner->EmitterTime, Context.GetDistributionData());
		if (CollisionPayload.Delay > Context.SpawnTime)
		{
			Particle.Flags |= STATE_Particle_DelayCollisions;
			Particle.Flags &= ~STATE_Particle_CollisionHasOccurred;
		}
	}
}

static float GParticleCollisionIgnoreInvisibleTime = .1f;
static FAutoConsoleVariableRef ParticleCollisionIgnoreInvisibleTime(
	TEXT("fx.ParticleCollisionIgnoreInvisibleTime"),
	GParticleCollisionIgnoreInvisibleTime,
	TEXT("The time a particle system component has to be invisible for to have all collision ignored. \n"),
	ECVF_Default
	);

void UParticleModuleCollision::Update(const FUpdateContext& Context)
{
	SCOPE_CYCLE_COUNTER(STAT_ParticleCollisionTime);
	FParticleEmitterInstance* Owner = &Context.Owner;
	UParticleSystemComponent* Component = Owner->Component.AsComponent();
	UWorld* World = Component->GetWorld();
	if (Owner->ActiveParticles == 0)
	{
		return;
	}

	//Gets the owning actor of the component. Can be NULL if the component is spawned with the World as an Outer, e.g. in UGameplayStatics::SpawnEmitterAtLocation().
	AActor* Actor = Component->GetOwner();

	UParticleLODLevel* LODLevel	= Owner->SpriteTemplate->GetCurrentLODLevel(Owner);
	check(LODLevel);

	const int32 MeshRotationOffset = Owner->GetMeshRotationOffset();
	const bool bMeshRotationActive = Owner->IsMeshRotationActive();

	const FTransform& OwnerTM = Component->GetAsyncComponentToWorld();
	const FVector ParentScale = OwnerTM.GetScale3D();

	FParticleEventInstancePayload* EventPayload = NULL;
	if (LODLevel->EventGenerator)
	{
		EventPayload = (FParticleEventInstancePayload*)(Owner->GetModuleInstanceData(LODLevel->EventGenerator));
		if (EventPayload && 
			(EventPayload->bCollisionEventsPresent == false) && 
			(EventPayload->bDeathEventsPresent == false))
		{
			EventPayload = NULL;
		}
	}

	FParticleCollisionInstancePayload* CollisionInstPayload = (FParticleCollisionInstancePayload*)(Owner->GetModuleInstanceData(this));

	const TArray<FVector>& PlayerLocations = Component->GetPlayerLocations();
	TArray<float> PlayerLODDistanceFactor = Component->GetPlayerLODDistanceFactor();	//Make a copy because we need to square it later
	const int32 PlayerCount = PlayerLocations.Num();

	if (World->IsGameWorld())
	{
		bool bIgnoreAllCollision = false;

		// LOD collision based on visibility
		// This is at the 'emitter instance' level as it will be true or false for the whole instance...
		if (bCollideOnlyIfVisible && ((World->TimeSeconds - Component->GetLastRenderTime()) > GParticleCollisionIgnoreInvisibleTime))
		{
			// no collision if not recently rendered
			bIgnoreAllCollision = true;
		}
		else
		{
			// If the MaxCollisionDistance is greater than WORLD_MAX, they obviously want the check disabled...
			if (MaxCollisionDistance < UE_OLD_WORLD_MAX)	// LWC_TODO: Fix needed to handle WORLD_MAX increase. Remove?
			{
				// If we have at least a few particles, do a simple check vs. the bounds
				if (Owner->ActiveParticles > 7)
				{
					if (CollisionInstPayload->CurrentLODBoundsCheckCount == 0)
					{
						FBox BoundingBox;
						BoundingBox.Init();
						if (Component->Template && Component->Template->bUseFixedRelativeBoundingBox)
						{
							BoundingBox = Component->Template->FixedRelativeBoundingBox.TransformBy(OwnerTM);
						}
						else
						{
							// A frame behind, but shouldn't be an issue...
							BoundingBox = Component->GetAsyncBounds().GetBox();
						}

						// see if any player is within the extended bounds...
						bIgnoreAllCollision = true;
						// Check for the system itself beyond beyond the bounds
						// LOD collision by distance
						bool bCloseEnough = false;
						for (int32 PlyrIdx = 0; PlyrIdx < PlayerCount; PlyrIdx++)
						{
							// Invert the LOD distance factor here because we are using it to *expand* the 
							// bounds rather than shorten the distance checked as it is usually used for.
							float InvDistanceFactor = 1.0f / PlayerLODDistanceFactor[PlyrIdx];
							FBox CheckBounds = BoundingBox;
							FBox::FReal BoxExpansionValue = MaxCollisionDistance * InvDistanceFactor;
							BoxExpansionValue += BoxExpansionValue * 0.075f;
							// Expand it by the max collision distance (and a little bit extra)
							CheckBounds = CheckBounds.ExpandBy(BoxExpansionValue);
							if (CheckBounds.IsInside(PlayerLocations[PlyrIdx]))
							{
								// If one is close enough, that's all it takes!
								bCloseEnough = true;
								break;
							}
						}
						if (bCloseEnough == true)
						{
							bIgnoreAllCollision = false;
						}
					}
					CollisionInstPayload->CurrentLODBoundsCheckCount++;
					// Every 30 frames recheck the overall bounds...
					if (CollisionInstPayload->CurrentLODBoundsCheckCount > 30)
					{
						CollisionInstPayload->CurrentLODBoundsCheckCount = 0;
					}
				}
			}
		}

		if (bIgnoreAllCollision == true)
		{
			// Turn off collision on *all* existing particles...
			// We don't want it to turn back on and have particles 
			// already embedded start performing collision checks.
			BEGIN_UPDATE_LOOP;
			{
				Particle.Flags |= STATE_Particle_IgnoreCollisions;
			}
			END_UPDATE_LOOP;
			return;
		}

		// Square the LODDistanceFactor values now, so we don't have to do it
		// per particle in the update loop below...
		for (int32 SquareIdx = 0; SquareIdx < PlayerLocations.Num(); SquareIdx++)
		{
			PlayerLODDistanceFactor[SquareIdx] *= PlayerLODDistanceFactor[SquareIdx];
		}
	}

	FVector::FReal SquaredMaxCollisionDistance = FMath::Square<FVector::FReal>(MaxCollisionDistance);
	BEGIN_UPDATE_LOOP;
	{
		if ((Particle.Flags & STATE_Particle_CollisionIgnoreCheck) != 0)
		{
			CONTINUE_UPDATE_LOOP;
		}

		PARTICLE_ELEMENT(FParticleCollisionPayload, CollisionPayload);
		if ((Particle.Flags & STATE_Particle_DelayCollisions) != 0)
		{
			if (CollisionPayload.Delay > Particle.RelativeTime)
			{
				CONTINUE_UPDATE_LOOP;
			}
			Particle.Flags &= ~STATE_Particle_DelayCollisions;
		}

		FVector			Location;
		FVector			OldLocation;

		// Location won't be calculated till after tick so we need to calculate an intermediate one here.
		Location	= Particle.Location + (FVector)Particle.Velocity * Context.DeltaTime;
		if (LODLevel->RequiredModule->bUseLocalSpace)
		{
			// Transform the location and old location into world space
			Location		= OwnerTM.TransformPosition(Location);
			OldLocation		= OwnerTM.TransformPosition(Particle.OldLocation);
		}
		else
		{
			OldLocation	= Particle.OldLocation;
		}
		FVector	Direction = (Location - OldLocation).GetSafeNormal();

		// Determine the size
		FVector Size = (FVector)Particle.Size * ParentScale;
		FVector	Extent(0.0f);

		// Setup extent for mesh particles. 
		UParticleModuleTypeDataMesh* MeshType = Cast<UParticleModuleTypeDataMesh>(LODLevel->TypeDataModule);
		if (MeshType && MeshType->Mesh)
		{
			Extent = MeshType->Mesh->GetBounds().BoxExtent;
			Extent = MeshType->bCollisionsConsiderPartilceSize ? Extent * Size : Extent;
		}
		
		FHitResult Hit;

		Hit.Normal.X = 0.0f;
		Hit.Normal.Y = 0.0f;
		Hit.Normal.Z = 0.0f;

		FVector End = Location + Direction * Size / DirScalar;

		if ((World->IsGameWorld() == true) && (MaxCollisionDistance < UE_OLD_WORLD_MAX))	// LWC_TODO: Fix needed to handle WORLD_MAX increase.
		{
			// LOD collision by distance
			bool bCloseEnough = false;
			for (int32 CheckIdx = 0; CheckIdx < PlayerCount; CheckIdx++)
			{
				float CheckValue = (PlayerLocations[CheckIdx] - End).SizeSquared() * PlayerLODDistanceFactor[CheckIdx];
				if (CheckValue < SquaredMaxCollisionDistance)
				{
					bCloseEnough = true;
					break;
				}
			}
			if (bCloseEnough == false)
			{
				Particle.Flags |= STATE_Particle_IgnoreCollisions;
				CONTINUE_UPDATE_LOOP;
			}
		}

		AActor* IgnoreActor = bIgnoreSourceActor ? Actor : NULL;

		if (PerformCollisionCheck(Owner, &Particle, Hit, IgnoreActor, End, OldLocation, Extent))
		{
			bool bDecrementMaxCount = true;
			bool bIgnoreCollision = false;
			if (Hit.HitObjectHandle.IsValid())
			{
				bDecrementMaxCount = !bPawnsDoNotDecrementCount || !Hit.HitObjectHandle.DoesRepresentClass(APawn::StaticClass());
				bIgnoreCollision = bIgnoreTriggerVolumes && Hit.HitObjectHandle.DoesRepresentClass(ATriggerBase::StaticClass());
				//@todo.SAS. Allow for PSys to say what it wants to collide w/?
			}

			if (bIgnoreCollision == false)
			{
				if (bDecrementMaxCount && (bOnlyVerticalNormalsDecrementCount == true))
				{
					if ((Hit.Normal.IsNearlyZero() == false) && (FMath::Abs(Hit.Normal.Z) + VerticalFudgeFactor) < 1.0f)
					{
						//UE_LOGF(LogParticles, Log, "Particle from %ls had a non-vertical hit!", *Context.GetTemplateName());
						bDecrementMaxCount = false;
					}
				}

				if (bDecrementMaxCount)
				{
					CollisionPayload.UsedCollisions--;
				}

				if (CollisionPayload.UsedCollisions > 0)
				{
					if (LODLevel->RequiredModule->bUseLocalSpace)
					{
						// Transform the particle velocity to world space
						FVector OldVelocity		= OwnerTM.TransformVector((FVector)Particle.Velocity);
						FVector	BaseVelocity	= OwnerTM.TransformVector((FVector)Particle.BaseVelocity);
						BaseVelocity			= BaseVelocity.MirrorByVector(Hit.Normal) * (FVector)CollisionPayload.UsedDampingFactor;

						Particle.BaseVelocity		= (FVector3f)OwnerTM.InverseTransformVector(BaseVelocity);
						Particle.BaseRotationRate	= Particle.BaseRotationRate * CollisionPayload.UsedDampingFactorRotation.X;
						if (bMeshRotationActive && MeshRotationOffset > 0)
						{
							FMeshRotationPayloadData* PayloadData = (FMeshRotationPayloadData*)((uint8*)&Particle + MeshRotationOffset);
							PayloadData->RotationRateBase *= CollisionPayload.UsedDampingFactorRotation;
						}

						// Reset the current velocity and manually adjust location to bounce off based on normal and time of collision.
						FVector NewVelocity	= Direction.MirrorByVector(Hit.Normal) * (Location - OldLocation).Size() * (FVector)CollisionPayload.UsedDampingFactor;
						Particle.Velocity		= FVector3f::ZeroVector;

						// New location
						FVector	NewLocation		= Location + NewVelocity * (1.f - Hit.Time);
						Particle.Location		= OwnerTM.InverseTransformPosition(NewLocation);

						if (bApplyPhysics)
						{
							check(IsInGameThread());
							UPrimitiveComponent* PrimitiveComponent = Hit.Component.Get();
							if(PrimitiveComponent && PrimitiveComponent->IsAnySimulatingPhysics())
							{
								FVector vImpulse;
								vImpulse = -(NewVelocity - OldVelocity) * ParticleMass.GetValue(Particle.RelativeTime, Context.GetDistributionData());
								PrimitiveComponent->AddImpulseAtLocation(vImpulse, Hit.Location, Hit.BoneName);
							}
						}
					}
					else
					{
						FVector vOldVelocity(Particle.Velocity);

						// Reflect base velocity and apply damping factor.
						Particle.BaseVelocity		= Particle.BaseVelocity.MirrorByVector((FVector3f)Hit.Normal) * CollisionPayload.UsedDampingFactor;
						Particle.BaseRotationRate	= Particle.BaseRotationRate * CollisionPayload.UsedDampingFactorRotation.X;
						if (bMeshRotationActive && MeshRotationOffset > 0)
						{
							FMeshRotationPayloadData* PayloadData = (FMeshRotationPayloadData*)((uint8*)&Particle + MeshRotationOffset);
							PayloadData->RotationRateBase *= CollisionPayload.UsedDampingFactorRotation;
						}

						// Reset the current velocity and manually adjust location to bounce off based on normal and time of collision.
						FVector vNewVelocity	= Direction.MirrorByVector(Hit.Normal) * (Location - OldLocation).Size() * (FVector)CollisionPayload.UsedDampingFactor;
						Particle.Velocity		= FVector3f::ZeroVector;
						Particle.Location	   += vNewVelocity * (1.f - Hit.Time);

						if (bApplyPhysics)
						{
							check(IsInGameThread());
							UPrimitiveComponent* PrimitiveComponent = Hit.Component.Get();
							if(PrimitiveComponent && PrimitiveComponent->IsAnySimulatingPhysics())
							{
								FVector vImpulse;
								vImpulse = -(vNewVelocity - vOldVelocity) * ParticleMass.GetValue(Particle.RelativeTime, Context.GetDistributionData());
								PrimitiveComponent->AddImpulseAtLocation(vImpulse, Hit.Location, Hit.BoneName);
							}
						}
					}

					if (EventPayload && (EventPayload->bCollisionEventsPresent == true))
					{
						LODLevel->EventGenerator->HandleParticleCollision(Owner, EventPayload, &CollisionPayload, &Hit, &Particle, Direction);
					}
				}
				else
				{
					if (LODLevel->RequiredModule->bUseLocalSpace == true)
					{
						Size = OwnerTM.TransformVector(Size);
					}
					Particle.Location = Hit.Location;
					if (LODLevel->RequiredModule->bUseLocalSpace == true)
					{
						// We need to transform the location back relative to the PSys.
						// NOTE: LocalSpace makes sense only for stationary emitters that use collision.
						Particle.Location = OwnerTM.InverseTransformPosition(Particle.Location);
					}
					switch (CollisionCompletionOption)
					{
					case EPCC_Kill:
						{
							if (EventPayload && (EventPayload->bDeathEventsPresent == true))
							{
								LODLevel->EventGenerator->HandleParticleKilled(Owner, EventPayload, &Particle);
							}
							KILL_CURRENT_PARTICLE;
						}
						break;
					case EPCC_Freeze:
						{
							Particle.Flags |= STATE_Particle_Freeze;
						}
						break;
					case EPCC_HaltCollisions:
						{
							Particle.Flags |= STATE_Particle_IgnoreCollisions;
						}
						break;
					case EPCC_FreezeTranslation:
						{
							Particle.Flags |= STATE_Particle_FreezeTranslation;
						}
						break;
					case EPCC_FreezeRotation:
						{
							Particle.Flags |= STATE_Particle_FreezeRotation;
						}
						break;
					case EPCC_FreezeMovement:
						{
							Particle.Flags |= STATE_Particle_FreezeRotation;
							Particle.Flags |= STATE_Particle_FreezeTranslation;
						}
						break;
					}

					if (EventPayload && (EventPayload->bCollisionEventsPresent == true))
					{
						LODLevel->EventGenerator->HandleParticleCollision(Owner, EventPayload, &CollisionPayload, &Hit, &Particle, Direction);
					}
				}
				Particle.Flags |= STATE_Particle_CollisionHasOccurred;
			}
		}
	}
	END_UPDATE_LOOP;
}

bool UParticleModuleCollision::CanTickInAnyThread()
{
	return !bApplyPhysics;
}

void UParticleModuleCollision::SetToSensibleDefaults(UParticleEmitter* Owner)
{
	UDistributionFloatUniform* MaxCollDist = Cast<UDistributionFloatUniform>(MaxCollisions.Distribution);
	if (MaxCollDist)
	{
		MaxCollDist->Min = 1.0f;
		MaxCollDist->Max = 1.0f;
		MaxCollDist->bIsDirty = true;
	}
}

bool UParticleModuleCollision::GenerateLODModuleValues(UParticleModule* SourceModule, float Percentage, UParticleLODLevel* LODLevel)
{
	// disable collision on emitters at the lowest LOD level
	//@todo.SAS. Determine how to forcibly disable collision now...
	return true;
}


bool UParticleModuleCollision::PerformCollisionCheck(FParticleEmitterInstance* Owner, FBaseParticle* InParticle, 
	FHitResult& Hit, AActor* SourceActor, const FVector& End, const FVector& Start, const FVector& Extent)
{
	UParticleSystemComponent* Component = Owner->Component.AsComponent();
	check(Component);
	return Component->ParticleLineCheck(Hit, SourceActor, End, Start, Extent, ObjectParams);
}

/*------------------------------------------------------------------------------
	GPU particle collision module.
------------------------------------------------------------------------------*/

UParticleModuleCollisionGPU::UParticleModuleCollisionGPU(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, Friction(0.0f)
	, RandomSpread(0.0f)
	, RandomDistribution(2.0f)
	, RadiusScale(1.0f)
	, RadiusBias(0.0f)
	, Response(EParticleCollisionResponse::Bounce)
{
	bSpawnModule = false;
	bUpdateModule = false;
}

void UParticleModuleCollisionGPU::InitializeDefaults()
{
	if (!Resilience.IsCreated())
	{
		UDistributionFloatConstant* ResilienceDistribution = NewObject<UDistributionFloatConstant>(this, TEXT("ResilienceDistribution"));
		ResilienceDistribution->Constant = 0.5f;
		Resilience.Distribution = ResilienceDistribution;
	}

	if (!ResilienceScaleOverLife.IsCreated())
	{
		UDistributionFloatConstant* ResilienceScaleOverLifeDistribution = NewObject<UDistributionFloatConstant>(this, TEXT("ResilienceScaleOverLifeDistribution"));
		ResilienceScaleOverLifeDistribution->Constant = 1.0f;
		ResilienceScaleOverLife.Distribution = ResilienceScaleOverLifeDistribution;
	}
}

void UParticleModuleCollisionGPU::PostInitProperties()
{
	Super::PostInitProperties();
	if (!HasAnyFlags(RF_ClassDefaultObject | RF_NeedLoad))
	{
		InitializeDefaults();
	}
}

#if WITH_EDITOR
void UParticleModuleCollisionGPU::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	InitializeDefaults();
	Super::PostEditChangeProperty(PropertyChangedEvent);
}
#endif // WITH_EDITOR

void UParticleModuleCollisionGPU::SetToSensibleDefaults(UParticleEmitter* Owner)
{
	UDistributionFloatConstant* ResilienceDistribution = Cast<UDistributionFloatConstant>(Resilience.Distribution);
	if (ResilienceDistribution)
	{
		ResilienceDistribution->Constant = 0.75f;
	}

	UDistributionFloatConstant* ResilienceScaleOverLifeDistribution = Cast<UDistributionFloatConstant>(ResilienceScaleOverLife.Distribution);
	if (ResilienceScaleOverLifeDistribution)
	{
		ResilienceScaleOverLifeDistribution->Constant = 1.0f;
	}
}

void UParticleModuleCollisionGPU::CompileModule(struct FParticleEmitterBuildInfo& EmitterInfo)
{
	EmitterInfo.bEnableCollision = true;
	EmitterInfo.CollisionMode = CollisionMode;
	EmitterInfo.CollisionResponse = Response;
	EmitterInfo.CollisionRadiusScale = RadiusScale;
	EmitterInfo.CollisionRadiusBias = RadiusBias;
	EmitterInfo.CollisionRandomSpread = RandomSpread;
	EmitterInfo.CollisionRandomDistribution = RandomDistribution;
	EmitterInfo.Friction = Friction;
	EmitterInfo.Resilience.Initialize(Resilience.Distribution);
	EmitterInfo.ResilienceScaleOverLife.Initialize(ResilienceScaleOverLife.Distribution);
}

#if WITH_EDITOR
bool UParticleModuleCollisionGPU::IsValidForLODLevel(UParticleLODLevel* LODLevel, FString& OutErrorString)
{
	check(LODLevel);

	UMaterialInterface* Material = NULL;
	if (LODLevel->RequiredModule)
	{
		Material = LODLevel->RequiredModule->Material;
	}
	if (Material == NULL)
	{
		Material = UMaterial::GetDefaultMaterial(MD_Surface);
	}
	check(Material);

	bool bIsOpaqueOrMasked = true;
	UWorld* World = GetWorld();
	const FMaterialResource* MaterialResource = Material->GetMaterialResource(World ? GetFeatureLevelShaderPlatform_Checked(World->GetFeatureLevel()) : GMaxRHIShaderPlatform);
	if (MaterialResource)
	{
		bIsOpaqueOrMasked = IsOpaqueOrMaskedBlendMode(*MaterialResource);
	}

	if (CollisionMode == EParticleCollisionMode::SceneDepth && bIsOpaqueOrMasked)
	{
		OutErrorString = NSLOCTEXT("UnrealEd", "CollisionOnOpaqueEmitter", "Scene depth collision cannot be used on emitters with an opaque material.").ToString();
		return false;
	}

	if (CollisionMode == EParticleCollisionMode::DistanceField)
	{
		if (DoesProjectSupportDistanceFields())
		{
			OutErrorString = NSLOCTEXT("UnrealEd", "CollisionWithoutDistanceField", "Distance Field collision requires the 'Generate Mesh Distance Fields' Renderer project setting to be enabled.").ToString();
			return false;
		}
	}

	if (LODLevel->TypeDataModule && LODLevel->TypeDataModule->IsA(UParticleModuleTypeDataGpu::StaticClass()))
	{
		if(!IsDistributionAllowedOnGPU(ResilienceScaleOverLife.Distribution))
		{
			OutErrorString = GetDistributionNotAllowedOnGPUText(StaticClass()->GetName(), "ResilienceScaleOverLife" ).ToString();
			return false;
		}
	}

	return true;
}
#endif
