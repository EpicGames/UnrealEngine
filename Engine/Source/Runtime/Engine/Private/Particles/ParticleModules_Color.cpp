// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	ParticleModules_Color.cpp: 
	Color-related particle module implementations.
=============================================================================*/

#include "Distributions/Distribution.h"
#include "UObject/UnrealType.h"
#include "ParticleEmitterInstances.h"
#include "UObject/Package.h"
#include "Particles/ParticleModule.h"
#include "Particles/ParticleSystem.h"
#include "Particles/ParticleSystemComponent.h"
#include "Distributions/DistributionFloatConstant.h"
#include "Distributions/DistributionVectorConstant.h"
#include "Distributions/DistributionFloatParticleParameter.h"
#include "Distributions/DistributionVectorParticleParameter.h"
#include "Particles/Color/ParticleModuleColorBase.h"
#include "Particles/Color/ParticleModuleColor.h"
#include "Particles/Color/ParticleModuleColorOverLife.h"
#include "Particles/Color/ParticleModuleColorScaleOverLife.h"
#include "Particles/Color/ParticleModuleColor_Seeded.h"
#include "Particles/ParticleEmitter.h"
#include "Particles/ParticleLODLevel.h"
#include "Distributions/DistributionVectorConstantCurve.h"
#include "Distributions/DistributionFloatConstantCurve.h"
#include "Engine/InterpCurveEdSetup.h"

UParticleModuleColorBase::UParticleModuleColorBase(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

/*-----------------------------------------------------------------------------
	Abstract base modules used for categorization.
-----------------------------------------------------------------------------*/


/*-----------------------------------------------------------------------------
	UParticleModuleColor implementation.
-----------------------------------------------------------------------------*/
UParticleModuleColor::UParticleModuleColor(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bSpawnModule = true;
	bUpdateModule = false;
	bCurvesAsColor = true;
	bClampAlpha = true;
}

void UParticleModuleColor::InitializeDefaults() 
{
	if (!StartColor.IsCreated())
	{
		StartColor.Distribution = NewObject<UDistributionVectorConstant>(this, TEXT("DistributionStartColor"));
	}

	if (!StartAlpha.IsCreated())
	{
		UDistributionFloatConstant* DistributionStartAlpha = NewObject<UDistributionFloatConstant>(this, TEXT("DistributionStartAlpha"));
		DistributionStartAlpha->Constant = 1.0f;
		StartAlpha.Distribution = DistributionStartAlpha;
	}
}

void UParticleModuleColor::PostInitProperties() 
{
	Super::PostInitProperties();
	if (!HasAnyFlags(RF_ClassDefaultObject | RF_NeedLoad))
	{
		InitializeDefaults();
	}
}

void UParticleModuleColor::CompileModule( FParticleEmitterBuildInfo& EmitterInfo )
{
	FVector InitialColor;
	float InitialAlpha;

	// Use a self-contained random number stream for compiling the module, so it doesn't differ between cooks.
	FRandomStream RandomStream(GetTypeHash(GetName()));
	InitialColor = StartColor.GetValue(0.0f, nullptr, 0, &RandomStream);
	InitialAlpha = StartAlpha.GetValue(0.0f, nullptr, &RandomStream);

	EmitterInfo.ColorScale.InitializeWithConstant( InitialColor );
	EmitterInfo.AlphaScale.InitializeWithConstant( InitialAlpha );
}

#if WITH_EDITOR
void UParticleModuleColor::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	InitializeDefaults();

	FProperty* PropertyThatChanged = PropertyChangedEvent.Property;
	if (PropertyThatChanged)
	{
		if (PropertyThatChanged->GetFName() == FName("bClampAlpha"))
		{
			UObject* OuterObj = GetOuter();
			check(OuterObj);
			UParticleLODLevel* LODLevel = Cast<UParticleLODLevel>(OuterObj);
			if (LODLevel)
			{
				// The outer is incorrect - warn the user and handle it
				UE_LOGF(LogParticles, Warning, "UParticleModuleColor has an incorrect outer... run FixupEmitters on package %ls",
					*(OuterObj->GetOutermost()->GetPathName()));
				OuterObj = LODLevel->GetOuter();
				UParticleEmitter* Emitter = Cast<UParticleEmitter>(OuterObj);
				check(Emitter);
				OuterObj = Emitter->GetOuter();
			}
			UParticleSystem* PartSys = CastChecked<UParticleSystem>(OuterObj);
			PartSys->UpdateColorModuleClampAlpha(this);
		}
	}
	Super::PostEditChangeProperty(PropertyChangedEvent);
}
#endif // WITH_EDITOR

bool UParticleModuleColor::AddModuleCurvesToEditor(UInterpCurveEdSetup* EdSetup, TArray<const FCurveEdEntry*>& OutCurveEntries)
{
	bool bNewCurve = false;
#if WITH_EDITORONLY_DATA
	// Iterate over object and find any InterpCurveFloats or UDistributionFloats
	for (TFieldIterator<FStructProperty> It(GetClass()); It; ++It)
	{
		// attempt to get a distribution from a random struct property
		UObject* Distribution = FRawDistribution::TryGetDistributionObjectFromRawDistributionProperty(*It, (uint8*)this);
		if (Distribution)
		{
			FCurveEdEntry* Curve = NULL;
			if(Distribution->IsA(UDistributionFloat::StaticClass()))
			{
				if (bClampAlpha == true)
				{
					bNewCurve |= EdSetup->AddCurveToCurrentTab(Distribution, It->GetName(), ModuleEditorColor, &Curve, true, true, true, 0.0f, 1.0f);
				}
				else
				{
					bNewCurve |= EdSetup->AddCurveToCurrentTab(Distribution, It->GetName(), ModuleEditorColor, &Curve, true, true);
				}
			}
			else
			{
				bNewCurve |= EdSetup->AddCurveToCurrentTab(Distribution, It->GetName(), ModuleEditorColor, &Curve, true, true);
			}
			OutCurveEntries.Add( Curve );
		}
	}
#endif // WITH_EDITORONLY_DATA
	return bNewCurve;
}

void UParticleModuleColor::Spawn(const FSpawnContext& Context)
{
	SpawnEx(Context, &GetRandomStream(Context));
}


void UParticleModuleColor::SpawnEx(const FSpawnContext& Context, struct FRandomStream* InRandomStream)
{
	FParticleEmitterInstance* Owner = &Context.Owner;

	SPAWN_INIT;
	{
		FVector3f ColorVec	= (FVector3f)StartColor.GetValue(Owner->EmitterTime, Context.GetDistributionData(), 0, InRandomStream);
		float	Alpha		= StartAlpha.GetValue(Owner->EmitterTime, Context.GetDistributionData(), InRandomStream);
		Particle_SetColorFromVector(ColorVec, Alpha, Particle.Color);
		Particle.BaseColor	= Particle.Color;
	}
}

void UParticleModuleColor::SetToSensibleDefaults(UParticleEmitter* Owner)
{
	UDistributionVectorConstant* StartColorDist = Cast<UDistributionVectorConstant>(StartColor.Distribution);
	if (StartColorDist)
	{
		StartColorDist->Constant = FVector(1.0f,1.0f,1.0f);
		StartColorDist->bIsDirty = true;
	}

}

/*-----------------------------------------------------------------------------
	UParticleModuleColor_Seeded implementation.
-----------------------------------------------------------------------------*/
UParticleModuleColor_Seeded::UParticleModuleColor_Seeded(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bSpawnModule = true;
	bSupportsRandomSeed = true;
	bRequiresLoopingNotification = true;
}

void UParticleModuleColor_Seeded::EmitterLoopingNotify(FParticleEmitterInstance* Owner)
{
	if (RandomSeedInfo.bResetSeedOnEmitterLooping == true)
	{
		FParticleRandomSeedInstancePayload* Payload = Owner->GetModuleRandomSeedInstanceData(this);
		PrepRandomSeedInstancePayload(Owner, Payload, RandomSeedInfo);
	}
}

/*-----------------------------------------------------------------------------
	UParticleModuleColorOverLife implementation.
-----------------------------------------------------------------------------*/
UParticleModuleColorOverLife::UParticleModuleColorOverLife(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bSpawnModule = true;
	bUpdateModule = true;
	bCurvesAsColor = true;
	bClampAlpha = true;
}

void UParticleModuleColorOverLife::InitializeDefaults()
{
	if (!ColorOverLife.IsCreated())
	{
		ColorOverLife.Distribution = NewObject<UDistributionVectorConstantCurve>(this, TEXT("DistributionColorOverLife"));
	}

	if (!AlphaOverLife.IsCreated())
	{
		UDistributionFloatConstant* DistributionAlphaOverLife = NewObject<UDistributionFloatConstant>(this, TEXT("DistributionAlphaOverLife"));
		DistributionAlphaOverLife->Constant = 1.0f;
		AlphaOverLife.Distribution = DistributionAlphaOverLife;
	}
}

void UParticleModuleColorOverLife::PostInitProperties()
{
	Super::PostInitProperties();
	if (!HasAnyFlags(RF_ClassDefaultObject | RF_NeedLoad))
	{
		ColorOverLife.Distribution = NewObject<UDistributionVectorConstantCurve>(this, TEXT("DistributionColorOverLife"));

		UDistributionFloatConstant* DistributionAlphaOverLife = NewObject<UDistributionFloatConstant>(this, TEXT("DistributionAlphaOverLife"));
		DistributionAlphaOverLife->Constant = 1.0f;
		AlphaOverLife.Distribution = DistributionAlphaOverLife;
	}
}

void UParticleModuleColorOverLife::CompileModule( FParticleEmitterBuildInfo& EmitterInfo )
{
	bool ScaleColor = true;
	bool ScaleAlpha = true;
	if(IsUsedInGPUEmitter())
	{
		if( ColorOverLife.Distribution->IsA( UDistributionVectorParticleParameter::StaticClass() ) )
		{
			EmitterInfo.DynamicColor = ColorOverLife;
#if WITH_EDITOR
			EmitterInfo.DynamicColor.Distribution->bIsDirty = true;
			EmitterInfo.DynamicColor.Initialize();
#endif
			ScaleColor = false;
			EmitterInfo.ColorScale.InitializeWithConstant( FVector(1.0f, 1.0f, 1.0f) );
		}

		if( AlphaOverLife.Distribution->IsA( UDistributionFloatParticleParameter::StaticClass() ) )
		{
			EmitterInfo.DynamicAlpha = AlphaOverLife;
#if WITH_EDITOR
			EmitterInfo.DynamicAlpha.Distribution->bIsDirty = true;
			EmitterInfo.DynamicAlpha.Initialize();
#endif
			ScaleAlpha = false;
			EmitterInfo.AlphaScale.InitializeWithConstant(1.0f);
		}
	}

	if( ScaleColor )
	{
		EmitterInfo.ColorScale.Initialize( ColorOverLife.Distribution );
	}

	if( ScaleAlpha )
	{
		EmitterInfo.AlphaScale.Initialize( AlphaOverLife.Distribution );
	}
}

#if WITH_EDITOR
void UParticleModuleColorOverLife::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	InitializeDefaults();

	FProperty* PropertyThatChanged = PropertyChangedEvent.Property;
	if (PropertyThatChanged)
	{
		if (PropertyThatChanged->GetFName() == FName("bClampAlpha"))
		{
			UObject* OuterObj = GetOuter();
			check(OuterObj);
			UParticleLODLevel* LODLevel = Cast<UParticleLODLevel>(OuterObj);
			if (LODLevel)
			{
				// The outer is incorrect - warn the user and handle it
				UE_LOGF(LogParticles, Warning, "UParticleModuleColorOverLife has an incorrect outer... run FixupEmitters on package %ls",
					*(OuterObj->GetOutermost()->GetPathName()));
				OuterObj = LODLevel->GetOuter();
				UParticleEmitter* Emitter = Cast<UParticleEmitter>(OuterObj);
				check(Emitter);
				OuterObj = Emitter->GetOuter();
			}
			UParticleSystem* PartSys = CastChecked<UParticleSystem>(OuterObj);

			PartSys->UpdateColorModuleClampAlpha(this);
		}
	}
	Super::PostEditChangeProperty(PropertyChangedEvent);
}
#endif // WITH_EDITOR

bool UParticleModuleColorOverLife::AddModuleCurvesToEditor(UInterpCurveEdSetup* EdSetup, TArray<const FCurveEdEntry*>& OutCurveEntries)
{
	bool bNewCurve = false;
#if WITH_EDITORONLY_DATA
	// Iterate over object and find any InterpCurveFloats or UDistributionFloats
	for (TFieldIterator<FStructProperty> It(GetClass()); It; ++It)
	{
		// attempt to get a distribution from a random struct property
		UObject* Distribution = FRawDistribution::TryGetDistributionObjectFromRawDistributionProperty(*It, (uint8*)this);
		if (Distribution)
		{
			FCurveEdEntry* Curve = NULL;
			if(Distribution->IsA(UDistributionFloat::StaticClass()))
			{
				// We are assuming that this is the alpha...
				if (bClampAlpha == true)
				{
					bNewCurve |= EdSetup->AddCurveToCurrentTab(Distribution, It->GetName(), ModuleEditorColor, &Curve, true, true, true, 0.0f, 1.0f);
				}
				else
				{
					bNewCurve |= EdSetup->AddCurveToCurrentTab(Distribution, It->GetName(), ModuleEditorColor, &Curve, true, true);
				}
			}
			else
			{
				// We are assuming that this is the color...
				bNewCurve |= EdSetup->AddCurveToCurrentTab(Distribution, It->GetName(), ModuleEditorColor, &Curve, true, true);
			}
			OutCurveEntries.Add( Curve );
		}
	}
#endif // WITH_EDITORONLY_DATA
	return bNewCurve;
}

void UParticleModuleColorOverLife::Spawn(const FSpawnContext& Context)
{
	FParticleEmitterInstance* Owner = &Context.Owner;

	SPAWN_INIT;
	FVector ColorVec	= ColorOverLife.GetValue(Particle.RelativeTime, Context.GetDistributionData());
	float	fAlpha		= AlphaOverLife.GetValue(Particle.RelativeTime, Context.GetDistributionData());
	Particle.Color.R	 = UE_REAL_TO_FLOAT(ColorVec.X);
	Particle.BaseColor.R = UE_REAL_TO_FLOAT(ColorVec.X);
	Particle.Color.G	 = UE_REAL_TO_FLOAT(ColorVec.Y);
	Particle.BaseColor.G = UE_REAL_TO_FLOAT(ColorVec.Y);
	Particle.Color.B	 = UE_REAL_TO_FLOAT(ColorVec.Z);
	Particle.BaseColor.B = UE_REAL_TO_FLOAT(ColorVec.Z);
	Particle.Color.A	 = fAlpha;
	Particle.BaseColor.A = fAlpha;
}

void UParticleModuleColorOverLife::Update(const FUpdateContext& Context)
{
	FParticleEmitterInstance* Owner = &Context.Owner;
	if ((Owner == NULL) || (Owner->ActiveParticles <= 0) || 
		(Owner->ParticleData == NULL) || (Owner->ParticleIndices == NULL))
	{
		return;
	}
	const FRawDistribution* FastColorOverLife = ColorOverLife.GetFastRawDistribution();
	const FRawDistribution* FastAlphaOverLife = AlphaOverLife.GetFastRawDistribution();
	FPlatformMisc::Prefetch(Owner->ParticleData, (Owner->ParticleIndices[0] * Owner->ParticleStride));
	FPlatformMisc::Prefetch(Owner->ParticleData, (Owner->ParticleIndices[0] * Owner->ParticleStride) + PLATFORM_CACHE_LINE_SIZE);
	if( FastColorOverLife && FastAlphaOverLife )
	{
		// fast path
		BEGIN_UPDATE_LOOP;
		{
			FPlatformMisc::Prefetch(ParticleData, (ParticleIndices[i+1] * ParticleStride));
			FPlatformMisc::Prefetch(ParticleData, (ParticleIndices[i+1] * ParticleStride) + PLATFORM_CACHE_LINE_SIZE);
			FastColorOverLife->GetValue3None(Particle.RelativeTime, &Particle.Color.R);
			FastAlphaOverLife->GetValue1None(Particle.RelativeTime, &Particle.Color.A);
		}
		END_UPDATE_LOOP;
	}
	else
	{
		FVector ColorVec;
		float	fAlpha;
		BEGIN_UPDATE_LOOP;
		{
			ColorVec = ColorOverLife.GetValue(Particle.RelativeTime, Context.GetDistributionData());
			fAlpha = AlphaOverLife.GetValue(Particle.RelativeTime, Context.GetDistributionData());
			FPlatformMisc::Prefetch(ParticleData, (ParticleIndices[i+1] * ParticleStride));
			FPlatformMisc::Prefetch(ParticleData, (ParticleIndices[i+1] * ParticleStride) + PLATFORM_CACHE_LINE_SIZE);
			Particle.Color.R = UE_REAL_TO_FLOAT(ColorVec.X);
			Particle.Color.G = UE_REAL_TO_FLOAT(ColorVec.Y);
			Particle.Color.B = UE_REAL_TO_FLOAT(ColorVec.Z);
			Particle.Color.A = fAlpha;
		}
		END_UPDATE_LOOP;
	}
}

void UParticleModuleColorOverLife::SetToSensibleDefaults(UParticleEmitter* Owner)
{
	ColorOverLife.Distribution = NewObject<UDistributionVectorConstantCurve>(this);
	UDistributionVectorConstantCurve* ColorOverLifeDist = Cast<UDistributionVectorConstantCurve>(ColorOverLife.Distribution);
	if (ColorOverLifeDist)
	{
		// Add two points, one at time 0.0f and one at 1.0f
		for (int32 Key = 0; Key < 2; Key++)
		{
			int32	KeyIndex = ColorOverLifeDist->CreateNewKey(Key * 1.0f);
			for (int32 SubIndex = 0; SubIndex < 3; SubIndex++)
			{
				if (Key == 0)
				{
					ColorOverLifeDist->SetKeyOut(SubIndex, KeyIndex, 1.0f);
				}
				else
				{
					ColorOverLifeDist->SetKeyOut(SubIndex, KeyIndex, 0.0f);
				}
			}
		}
		ColorOverLifeDist->bIsDirty = true;
	}

	AlphaOverLife.Distribution = NewObject<UDistributionFloatConstantCurve>(this);
	UDistributionFloatConstantCurve* AlphaOverLifeDist = Cast<UDistributionFloatConstantCurve>(AlphaOverLife.Distribution);
	if (AlphaOverLifeDist)
	{
		// Add two points, one at time 0.0f and one at 1.0f
		for (int32 Key = 0; Key < 2; Key++)
		{
			int32	KeyIndex = AlphaOverLifeDist->CreateNewKey(Key * 1.0f);
			if (Key == 0)
			{
				AlphaOverLifeDist->SetKeyOut(0, KeyIndex, 1.0f);
			}
			else
			{
				AlphaOverLifeDist->SetKeyOut(0, KeyIndex, 0.0f);
			}
		}
		AlphaOverLifeDist->bIsDirty = true;
	}
}

/*-----------------------------------------------------------------------------
	UParticleModuleColorScaleOverLife implementation.
-----------------------------------------------------------------------------*/
UParticleModuleColorScaleOverLife::UParticleModuleColorScaleOverLife(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bSpawnModule = true;
	bUpdateModule = true;
	bCurvesAsColor = true;
}

void UParticleModuleColorScaleOverLife::InitializeDefaults()
{
	if (!ColorScaleOverLife.IsCreated())
	{
		ColorScaleOverLife.Distribution = NewObject<UDistributionVectorConstantCurve>(this, TEXT("DistributionColorScaleOverLife"));
	}

	if (!AlphaScaleOverLife.IsCreated())
	{
		UDistributionFloatConstant* DistributionAlphaScaleOverLife = NewObject<UDistributionFloatConstant>(this, TEXT("DistributionAlphaScaleOverLife"));
		DistributionAlphaScaleOverLife->Constant = 1.0f;
		AlphaScaleOverLife.Distribution = DistributionAlphaScaleOverLife;
	}
}

void UParticleModuleColorScaleOverLife::PostInitProperties()
{
	Super::PostInitProperties();
	if (!HasAnyFlags(RF_ClassDefaultObject | RF_NeedLoad))
	{
		InitializeDefaults();
	}
}

void UParticleModuleColorScaleOverLife::CompileModule( FParticleEmitterBuildInfo& EmitterInfo )
{
	bool ScaleColor = true;
	bool ScaleAlpha = true;

	if(IsUsedInGPUEmitter())
	{
		if( ColorScaleOverLife.Distribution->IsA( UDistributionVectorParticleParameter::StaticClass() ) )
		{
			EmitterInfo.DynamicColorScale = ColorScaleOverLife;
#if WITH_EDITOR
			EmitterInfo.DynamicColorScale.Distribution->bIsDirty = true;
			EmitterInfo.DynamicColorScale.Initialize();
#endif
			ScaleColor = false;
		}

		if( AlphaScaleOverLife.Distribution->IsA( UDistributionFloatParticleParameter::StaticClass() ) )
		{
			EmitterInfo.DynamicAlphaScale = AlphaScaleOverLife;
#if WITH_EDITOR
			EmitterInfo.DynamicAlphaScale.Distribution->bIsDirty = true;
			EmitterInfo.DynamicAlphaScale.Initialize();
#endif
			ScaleAlpha = false;
		}
	}

	if( ScaleColor )
	{
		EmitterInfo.ColorScale.ScaleByVectorDistribution( ColorScaleOverLife.Distribution );
	}
	
	if( ScaleAlpha )
	{
		EmitterInfo.AlphaScale.ScaleByDistribution( AlphaScaleOverLife.Distribution );
	}
}

#if WITH_EDITOR
void UParticleModuleColorScaleOverLife::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	InitializeDefaults();
	Super::PostEditChangeProperty(PropertyChangedEvent);
}
#endif // WITH_EDITOR

void UParticleModuleColorScaleOverLife::Spawn(const FSpawnContext& Context)
{
	FParticleEmitterInstance* Owner = &Context.Owner;

	SPAWN_INIT;
	FVector ColorVec;
	float	fAlpha;

	if (bEmitterTime)
	{
		ColorVec	= ColorScaleOverLife.GetValue(Owner->EmitterTime, Context.GetDistributionData());
		fAlpha		= AlphaScaleOverLife.GetValue(Owner->EmitterTime, Context.GetDistributionData());
	}
	else
	{
		ColorVec	= ColorScaleOverLife.GetValue(Particle.RelativeTime, Context.GetDistributionData());
		fAlpha		= AlphaScaleOverLife.GetValue(Particle.RelativeTime, Context.GetDistributionData());
	}

	Particle.Color.R *= UE_REAL_TO_FLOAT(ColorVec.X);
	Particle.Color.G *= UE_REAL_TO_FLOAT(ColorVec.Y);
	Particle.Color.B *= UE_REAL_TO_FLOAT(ColorVec.Z);
	Particle.Color.A *= fAlpha;
}

void UParticleModuleColorScaleOverLife::Update(const FUpdateContext& Context)
{ 
	FParticleEmitterInstance* Owner = &Context.Owner;
	const FRawDistribution* FastColorScaleOverLife = ColorScaleOverLife.GetFastRawDistribution();
	const FRawDistribution* FastAlphaScaleOverLife = AlphaScaleOverLife.GetFastRawDistribution();
	FVector3f ColorVec;	// LWC_TODO: WRONG? Force ColorVec to floats for serialization. Particle.Color is FVector. Update to FVector3f also?
	float	fAlpha;
	if( FastColorScaleOverLife && FastAlphaScaleOverLife )
	{
		// fast path
		if (bEmitterTime)
		{
			BEGIN_UPDATE_LOOP;
			{
				FastColorScaleOverLife->GetValue3None(Owner->EmitterTime, &ColorVec.X);
				FastAlphaScaleOverLife->GetValue1None(Owner->EmitterTime, &fAlpha);
				Particle.Color.R *= ColorVec.X;
				Particle.Color.G *= ColorVec.Y;
				Particle.Color.B *= ColorVec.Z;
				Particle.Color.A *= fAlpha;
			}
			END_UPDATE_LOOP;
		}
		else
		{
			BEGIN_UPDATE_LOOP;
			{
				FastColorScaleOverLife->GetValue3None(Particle.RelativeTime, &ColorVec.X);
				FastAlphaScaleOverLife->GetValue1None(Particle.RelativeTime, &fAlpha);
				Particle.Color.R *= ColorVec.X;
				Particle.Color.G *= ColorVec.Y;
				Particle.Color.B *= ColorVec.Z;
				Particle.Color.A *= fAlpha;
			}
			END_UPDATE_LOOP;
		}
	}
	else
	{
		if (bEmitterTime)
		{
			BEGIN_UPDATE_LOOP;
				ColorVec = FVector3f(ColorScaleOverLife.GetValue(Owner->EmitterTime, Context.GetDistributionData()));
				fAlpha = AlphaScaleOverLife.GetValue(Owner->EmitterTime, Context.GetDistributionData());
				Particle.Color.R *= ColorVec.X;
				Particle.Color.G *= ColorVec.Y;
				Particle.Color.B *= ColorVec.Z;
				Particle.Color.A *= fAlpha;
			END_UPDATE_LOOP;
		}
		else
		{
			BEGIN_UPDATE_LOOP;
				ColorVec = FVector3f(ColorScaleOverLife.GetValue(Particle.RelativeTime, Context.GetDistributionData()));
				fAlpha = AlphaScaleOverLife.GetValue(Particle.RelativeTime, Context.GetDistributionData());
				Particle.Color.R *= ColorVec.X;
				Particle.Color.G *= ColorVec.Y;
				Particle.Color.B *= ColorVec.Z;
				Particle.Color.A *= fAlpha;
			END_UPDATE_LOOP;
		}
	}
}

void UParticleModuleColorScaleOverLife::SetToSensibleDefaults(UParticleEmitter* Owner)
{
	ColorScaleOverLife.Distribution = NewObject<UDistributionVectorConstantCurve>(this);
	UDistributionVectorConstantCurve* ColorScaleOverLifeDist = Cast<UDistributionVectorConstantCurve>(ColorScaleOverLife.Distribution);
	if (ColorScaleOverLifeDist)
	{
		// Add two points, one at time 0.0f and one at 1.0f
		for (int32 Key = 0; Key < 2; Key++)
		{
			int32	KeyIndex = ColorScaleOverLifeDist->CreateNewKey(Key * 1.0f);
			for (int32 SubIndex = 0; SubIndex < 3; SubIndex++)
			{
				ColorScaleOverLifeDist->SetKeyOut(SubIndex, KeyIndex, 1.0f);
			}
		}
		ColorScaleOverLifeDist->bIsDirty = true;
	}
}

#if WITH_EDITOR
int32 UParticleModuleColorScaleOverLife::GetNumberOfCustomMenuOptions() const
{
	return 1;
}

bool UParticleModuleColorScaleOverLife::GetCustomMenuEntryDisplayString(int32 InEntryIndex, FString& OutDisplayString) const
{
	if (InEntryIndex == 0)
	{
		OutDisplayString = NSLOCTEXT("UnrealEd", "Module_ColorScaleOverLife_SetupParticleParam", "Set up particle parameter").ToString();
		return true;
	}
	return false;
}

bool UParticleModuleColorScaleOverLife::PerformCustomMenuEntry(int32 InEntryIndex)
{
	if (GIsEditor == true)
	{
		if (InEntryIndex == 0)
		{
			UE_LOGF(LogParticles, Log, "Setup color scale over life for particle param!");
			ColorScaleOverLife.Distribution = NewObject<UDistributionVectorParticleParameter>(this);
			UDistributionVectorParticleParameter* ColorDist = Cast<UDistributionVectorParticleParameter>(ColorScaleOverLife.Distribution);
			if (ColorDist)
			{
				ColorDist->ParameterName = FName("InstanceColorScaleOverLife");
				ColorDist->ParamModes[0] = DPM_Direct;
				ColorDist->ParamModes[1] = DPM_Direct;
				ColorDist->ParamModes[2] = DPM_Direct;
				ColorDist->Constant = FVector(1.0f);
				ColorDist->bIsDirty = true;
			}

			AlphaScaleOverLife.Distribution = NewObject<UDistributionFloatParticleParameter>(this);
			UDistributionFloatParticleParameter* AlphaDist = Cast<UDistributionFloatParticleParameter>(AlphaScaleOverLife.Distribution);
			if (AlphaDist)
			{
				AlphaDist->ParameterName = FName("InstanceAlphaScaleOverLife");
				AlphaDist->ParamMode = DPM_Direct;
				AlphaDist->Constant = 1.0f;
				AlphaDist->bIsDirty = true;
			}
		}
		return true;
	}
	return false;
}

#endif
