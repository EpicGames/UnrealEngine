// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	ParticleModules_Event.cpp: Particle event-related module implementations.
=============================================================================*/

#include "Engine/HitResult.h"
#include "Particles/ParticleSystem.h"
#include "Distributions/DistributionFloatConstant.h"
#include "Distributions/DistributionVectorConstant.h"
#include "ParticleEmitterInstances.h"
#include "ParticleEmitterInstanceOwner.h"
#include "Particles/Event/ParticleModuleEventBase.h"
#include "Particles/ParticleEmitter.h"
#include "Particles/Event/ParticleModuleEventGenerator.h"
#include "Particles/Event/ParticleModuleEventReceiverBase.h"
#include "Particles/Event/ParticleModuleEventReceiverKillParticles.h"
#include "Particles/Event/ParticleModuleEventReceiverSpawn.h"
#include "Particles/Event/ParticleModuleEventSendToGame.h"
#include "Particles/ParticleLODLevel.h"
#include "Particles/ParticleModule.h"
#include "Particles/ParticleSystemComponent.h"

/*-----------------------------------------------------------------------------
	Abstract base modules used for categorization.
-----------------------------------------------------------------------------*/

/*-----------------------------------------------------------------------------
	UParticleModuleEventSendToGame implementation.
-----------------------------------------------------------------------------*/
UParticleModuleEventSendToGame::UParticleModuleEventSendToGame(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

/*-----------------------------------------------------------------------------
	UParticleModuleEventBase implementation.
-----------------------------------------------------------------------------*/
UParticleModuleEventBase::UParticleModuleEventBase(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

/*-----------------------------------------------------------------------------
	UParticleModuleEventGenerator implementation.
-----------------------------------------------------------------------------*/
UParticleModuleEventGenerator::UParticleModuleEventGenerator(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bSpawnModule = true;
	bUpdateModule = true;
}

void UParticleModuleEventGenerator::Spawn(const FSpawnContext& Context)
{
}

void UParticleModuleEventGenerator::Update(const FUpdateContext& Context)
{
}

uint32 UParticleModuleEventGenerator::RequiredBytes(UParticleModuleTypeDataBase* TypeData)
{
	return 0;
}

uint32 UParticleModuleEventGenerator::RequiredBytesPerInstance()
{
	return sizeof(FParticleEventInstancePayload);
}

uint32 UParticleModuleEventGenerator::PrepPerInstanceBlock(FParticleEmitterInstance* Owner, void* InstData)
{
	FParticleEventInstancePayload* Payload = (FParticleEventInstancePayload*)InstData;
	if (Payload)
	{
		for (int32 EventGenIndex = 0; EventGenIndex < Events.Num(); EventGenIndex++)
		{
			switch (Events[EventGenIndex].Type)
			{
			case EPET_Spawn:		Payload->bSpawnEventsPresent = true;		break;
			case EPET_Death:		Payload->bDeathEventsPresent = true;		break;
			case EPET_Collision:	Payload->bCollisionEventsPresent = true;	break;
			case EPET_Burst:		Payload->bBurstEventsPresent = true;		break;
			}
		}
		return 0;
	}

	return 0xffffffff;
}

#if WITH_EDITOR
void UParticleModuleEventGenerator::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);
	UObject* OuterObj = GetOuter();
	check(OuterObj);
	UParticleLODLevel* LODLevel = Cast<UParticleLODLevel>(OuterObj);
	if (LODLevel)
	{
		// The outer is incorrect - warn the user and handle it
		OuterObj = LODLevel->GetOuter();
		UParticleEmitter* Emitter = Cast<UParticleEmitter>(OuterObj);
		check(Emitter);
		OuterObj = Emitter->GetOuter();
	}
	UParticleSystem* PartSys = CastChecked<UParticleSystem>(OuterObj);
	if (PartSys)
	{
		PartSys->PostEditChangeProperty(PropertyChangedEvent);
	}
}
#endif // WITH_EDITOR

bool UParticleModuleEventGenerator::HandleParticleSpawned(FParticleEmitterInstance* Owner, 
	FParticleEventInstancePayload* EventPayload, FBaseParticle* NewParticle)
{
	check(Owner && EventPayload && NewParticle);

	EventPayload->SpawnTrackingCount++;

	bool bProcessed = false;
	for (int32 EventIndex = 0; EventIndex < Events.Num(); EventIndex++)
	{
		FParticleEvent_GenerateInfo& EventGenInfo = Events[EventIndex];
		if (EventGenInfo.Type == EPET_Spawn)
		{
			if (EventGenInfo.Frequency == 0 || (EventPayload->SpawnTrackingCount % EventGenInfo.Frequency) == 0)
			{
				FVector ParticleLocation = EventGenInfo.bUseOrbitOffset ? Owner->GetParticleLocationWithOrbitOffset(NewParticle) : FVector(NewParticle->Location);

				Owner->Component.ReportEventSpawn(EventGenInfo.CustomName, Owner->EmitterTime, 
					ParticleLocation, FVector(NewParticle->Velocity), EventGenInfo.ParticleModuleEventsToSendToGame);
				bProcessed = true;
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
				Owner->EventCount++;
#endif	//#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
			}
		}
	}

	return bProcessed;
}

bool UParticleModuleEventGenerator::HandleParticleKilled(FParticleEmitterInstance* Owner, 
	FParticleEventInstancePayload* EventPayload, FBaseParticle* DeadParticle)
{
	check(Owner && EventPayload && DeadParticle);

	EventPayload->DeathTrackingCount++;

	bool bProcessed = false;
	for (int32 EventIndex = 0; EventIndex < Events.Num(); EventIndex++)
	{
		FParticleEvent_GenerateInfo& EventGenInfo = Events[EventIndex];
		if (EventGenInfo.Type == EPET_Death)
		{
			if (EventGenInfo.Frequency == 0 || (EventPayload->DeathTrackingCount % EventGenInfo.Frequency) == 0)
			{
				FVector ParticleLocation = EventGenInfo.bUseOrbitOffset ? Owner->GetParticleLocationWithOrbitOffset(DeadParticle) : FVector(DeadParticle->Location);

				Owner->Component.ReportEventDeath(EventGenInfo.CustomName, 
					Owner->EmitterTime, ParticleLocation, (FVector)DeadParticle->Velocity,
					EventGenInfo.ParticleModuleEventsToSendToGame, DeadParticle->RelativeTime);
				bProcessed = true;
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
				Owner->EventCount++;
#endif	//#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
			}
		}
	}

	return bProcessed;
}

bool UParticleModuleEventGenerator::HandleParticleCollision(FParticleEmitterInstance* Owner, 
	FParticleEventInstancePayload* EventPayload, FParticleCollisionPayload* CollidePayload, 
	FHitResult* Hit, FBaseParticle* CollideParticle, FVector& CollideDirection)
{
	check(Owner && EventPayload && CollideParticle);

	EventPayload->CollisionTrackingCount++;

	bool bProcessed = false;
	for (int32 EventIndex = 0; EventIndex < Events.Num(); EventIndex++)
	{
		FParticleEvent_GenerateInfo& EventGenInfo = Events[EventIndex];
		if (EventGenInfo.Type == EPET_Collision)
		{
			if (EventGenInfo.FirstTimeOnly == true)
			{
				if ((CollideParticle->Flags & STATE_Particle_CollisionHasOccurred) != 0)
				{
					continue;
				}
			}
			else
			if (EventGenInfo.LastTimeOnly == true)
			{
				if (CollidePayload->UsedCollisions != 0)
				{
					continue;
				}
			}

			if (EventGenInfo.Frequency == 0 || (EventPayload->CollisionTrackingCount % EventGenInfo.Frequency) == 0)
			{
				Owner->Component.ReportEventCollision(
					EventGenInfo.CustomName, 
					Owner->EmitterTime, 
					Hit->Location,
					CollideDirection, 
					(FVector)CollideParticle->Velocity,
					EventGenInfo.ParticleModuleEventsToSendToGame,
					CollideParticle->RelativeTime, 
					Hit->Normal, 
					Hit->Time, 
					Hit->Item, 
					Hit->BoneName,
					Hit->PhysMaterial.Get());
				bProcessed = true;
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
				Owner->EventCount++;
#endif	//#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
			}
		}
	}

	return bProcessed;
}

bool UParticleModuleEventGenerator::HandleParticleBurst(FParticleEmitterInstance* Owner, 
	FParticleEventInstancePayload* EventPayload, const int32 ParticleCount)
{
	check(Owner && EventPayload);

	++EventPayload->BurstTrackingCount;

	bool bProcessed = false;
	for (int32 EventIndex = 0; EventIndex < Events.Num(); ++EventIndex)
	{
		FParticleEvent_GenerateInfo& EventGenInfo = Events[EventIndex];
		if (EventGenInfo.Type == EPET_Burst)
		{
			if (EventGenInfo.Frequency == 0 || (EventPayload->BurstTrackingCount % EventGenInfo.Frequency) == 0)
			{
				Owner->Component.ReportEventBurst(EventGenInfo.CustomName, Owner->EmitterTime, ParticleCount, 
					Owner->Location, EventGenInfo.ParticleModuleEventsToSendToGame);
				bProcessed = true;
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
				Owner->EventCount++;
#endif	//#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
			}
		}
	}

	return bProcessed;
}

/*-----------------------------------------------------------------------------
	UParticleModuleEventReceiverBase implementation.
-----------------------------------------------------------------------------*/
UParticleModuleEventReceiverBase::UParticleModuleEventReceiverBase(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

/*-----------------------------------------------------------------------------
	UParticleModuleEventReceiver implementation.
-----------------------------------------------------------------------------*/
UParticleModuleEventReceiverKillParticles::UParticleModuleEventReceiverKillParticles(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bSpawnModule = true;
	bUpdateModule = true;
}

bool UParticleModuleEventReceiverKillParticles::ProcessParticleEvent(FParticleEmitterInstance* Owner, FParticleEventData& InEvent, float DeltaTime)
{
	if ((InEvent.EventName == EventName) && ((EventGeneratorType == EPET_Any) || (EventGeneratorType == InEvent.Type)))
	{
		Owner->KillParticlesForced(true);
		if (bStopSpawning == true)
		{
			Owner->SetHaltSpawning(true);
			Owner->SetHaltSpawningExternal(true);
		}
		return true;
	}

	return false;
}

/*-----------------------------------------------------------------------------
	UParticleModuleEventReceiver implementation.
-----------------------------------------------------------------------------*/
UParticleModuleEventReceiverSpawn::UParticleModuleEventReceiverSpawn(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bSpawnModule = true;
	bUpdateModule = true;
}

void UParticleModuleEventReceiverSpawn::InitializeDefaults()
{
	if (!SpawnCount.IsCreated())
	{
		UDistributionFloatConstant* RequiredDistributionSpawnCount = NewObject<UDistributionFloatConstant>(this, TEXT("RequiredDistributionSpawnCount"));
		RequiredDistributionSpawnCount->Constant = 0.0f;
		SpawnCount.Distribution = RequiredDistributionSpawnCount;
	}

	if (!InheritVelocityScale.IsCreated())
	{
		UDistributionVectorConstant* RequiredDistributionInheritVelocityScale = NewObject<UDistributionVectorConstant>(this, TEXT("RequiredDistributionInheritVelocityScale"));
		RequiredDistributionInheritVelocityScale->Constant = FVector(1.0f, 1.0f, 1.0f);
		InheritVelocityScale.Distribution = RequiredDistributionInheritVelocityScale;
	}
}

void UParticleModuleEventReceiverSpawn::PostInitProperties()
{
	Super::PostInitProperties();
	if (!HasAnyFlags(RF_ClassDefaultObject | RF_NeedLoad))
	{
		InitializeDefaults();
	}
}

#if WITH_EDITOR
void UParticleModuleEventReceiverSpawn::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	InitializeDefaults();
	Super::PostEditChangeProperty(PropertyChangedEvent);
}
#endif // WITH_EDITOR

bool UParticleModuleEventReceiverSpawn::ProcessParticleEvent(FParticleEmitterInstance* Owner, FParticleEventData& InEvent, float DeltaTime)
{
	if ((InEvent.EventName == EventName) && ((EventGeneratorType == EPET_Any) || (EventGeneratorType == InEvent.Type)))
	{
		int32 Count = 0;

		switch (InEvent.Type)
		{
		case EPET_Spawn:
		case EPET_Burst:
			Count = FMath::RoundToInt(SpawnCount.GetValue(InEvent.EmitterTime));
			break;
		case EPET_Death:
			{
				FParticleEventDeathData* DeathData = (FParticleEventDeathData*)(&InEvent);
				Count = FMath::RoundToInt(SpawnCount.GetValue(bUseParticleTime ? DeathData->ParticleTime : InEvent.EmitterTime));
			}
			break;
		case EPET_Collision:
			{
				FParticleEventCollideData* CollideData = (FParticleEventCollideData*)(&InEvent);
				UPhysicalMaterial* PhysMat = CollideData->PhysMat;
				bool bPhysMatIsAllowed = !PhysMat || (PhysicalMaterials.Num() == 0 || PhysicalMaterials.Contains(PhysMat) == !bBanPhysicalMaterials);

				if (bPhysMatIsAllowed)
				{
					Count = FMath::RoundToInt(SpawnCount.GetValue(bUseParticleTime ? CollideData->ParticleTime : InEvent.EmitterTime));
				}
			}
			break;
		case EPET_Blueprint:
			{
				FParticleEventKismetData* KismetData = (FParticleEventKismetData*)(&InEvent);
				Count = FMath::RoundToInt(SpawnCount.GetValue(InEvent.EmitterTime));
			}
			break;
		}

		if (Count > 0)
		{
			FVector SpawnLocation = bUsePSysLocation ? Owner->Location : InEvent.Location;
			FVector Velocity = bInheritVelocity ? 
				(InEvent.Velocity * InheritVelocityScale.GetValue(InEvent.EmitterTime)) : FVector::ZeroVector;
			
			Owner->ForceSpawn(DeltaTime, 0, Count, SpawnLocation, Velocity);
		}

		return true;
	}

	return false;
}
