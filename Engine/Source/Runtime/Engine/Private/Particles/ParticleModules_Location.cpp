// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	ParticleModules_Location.cpp: 
	Location-related particle module implementations.
=============================================================================*/

#include "Misc/MessageDialog.h"
#include "Components/SkeletalMeshComponent.h"
#include "ParticleEmitterInstances.h"
#include "ParticleEmitterInstanceOwner.h"
#include "Particles/ParticleSystemComponent.h"
#include "Distributions/DistributionFloatConstant.h"
#include "Distributions/DistributionVectorConstant.h"
#include "Distributions/DistributionVectorUniform.h"
#include "Distributions/DistributionVectorConstantCurve.h"
#include "Particles/Location/ParticleModuleLocationBase.h"
#include "Particles/Location/ParticleModuleLocation.h"
#include "Particles/Location/ParticleModuleLocationBoneSocket.h"
#include "Particles/Location/ParticleModuleLocationDirect.h"
#include "Particles/Location/ParticleModuleLocationEmitter.h"
#include "Particles/Location/ParticleModuleLocationEmitterDirect.h"
#include "Particles/Location/ParticleModuleLocationPrimitiveBase.h"
#include "Particles/Location/ParticleModuleLocationPrimitiveCylinder.h"
#include "Particles/Location/ParticleModuleLocationPrimitiveCylinder_Seeded.h"
#include "Particles/Location/ParticleModuleLocationPrimitiveSphere.h"
#include "Particles/Location/ParticleModuleLocationPrimitiveSphere_Seeded.h"
#include "Particles/Location/ParticleModuleLocationPrimitiveTriangle.h"
#include "Particles/Location/ParticleModuleLocationSkelVertSurface.h"
#include "Particles/Location/ParticleModuleLocationWorldOffset.h"
#include "Particles/Location/ParticleModuleLocationWorldOffset_Seeded.h"
#include "Particles/Location/ParticleModuleLocation_Seeded.h"
#include "Particles/ParticleEmitter.h"
#include "Particles/TypeData/ParticleModuleTypeDataGpu.h"
#include "Particles/ParticleLODLevel.h"
#include "Particles/ParticleModule.h"
#include "Particles/ParticleModuleRequired.h"
#include "Animation/SkeletalMeshActor.h"
#include "Engine/SkeletalMesh.h"
#include "Engine/SkeletalMeshSocket.h"
#include "Particles/ParticleSystem.h"
#include "Rendering/SkeletalMeshRenderData.h"
#include "Particles/TypeData/ParticleModuleTypeDataBase.h"
#include "PrimitiveDrawingUtils.h"
#include "UObject/UnrealType.h"


UParticleModuleLocationBase::UParticleModuleLocationBase(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}


/*-----------------------------------------------------------------------------
	Abstract base modules used for categorization.
-----------------------------------------------------------------------------*/

/*-----------------------------------------------------------------------------
	UParticleModuleLocation implementation.
-----------------------------------------------------------------------------*/

UParticleModuleLocation::UParticleModuleLocation(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bSpawnModule = true;
	bSupported3DDrawMode = true;
	DistributeOverNPoints = 0.0f;
}

void UParticleModuleLocation::InitializeDefaults()
{
	if (!StartLocation.IsCreated())
	{
		StartLocation.Distribution = NewObject<UDistributionVectorUniform>(this, TEXT("DistributionStartLocation"));
	}
}

void UParticleModuleLocation::PostInitProperties()
{
	Super::PostInitProperties();
	if (!HasAnyFlags(RF_ClassDefaultObject | RF_NeedLoad))
	{
		InitializeDefaults();
	}
}

#if WITH_EDITOR
void UParticleModuleLocation::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	InitializeDefaults();
	Super::PostEditChangeProperty(PropertyChangedEvent);
}
#endif // WITH_EDITOR

void UParticleModuleLocation::Spawn(const FSpawnContext& Context)
{
	SpawnEx(Context, &GetRandomStream(Context));
}

void UParticleModuleLocation::SpawnEx(const FSpawnContext& Context, struct FRandomStream* InRandomStream)
{
	SPAWN_INIT;
	FParticleEmitterInstance* Owner = &Context.Owner;
	UParticleLODLevel* LODLevel	= Owner->SpriteTemplate->GetCurrentLODLevel(Owner);
	check(LODLevel);
	FVector LocationOffset;

	// Avoid divide by zero.
	if ((DistributeOverNPoints != 0.0f) && (DistributeOverNPoints != 1.f))
	{
		float RandomNum = InRandomStream->FRand() * FMath::Fractional(Owner->EmitterTime);

		if(RandomNum > DistributeThreshold)
		{
			LocationOffset = StartLocation.GetValue(Owner->EmitterTime, Context.GetDistributionData(), 0, InRandomStream);
		}
		else
		{
			FVector Min, Max;
			StartLocation.GetRange(Min, Max);
			FVector Lerped = FMath::Lerp(Min, Max, FMath::TruncToFloat((InRandomStream->FRand() * (DistributeOverNPoints - 1.0f)) + 0.5f)/(DistributeOverNPoints - 1.0f));
			LocationOffset.Set(Lerped.X, Lerped.Y, Lerped.Z);
		}
	}
	else
	{
		LocationOffset = StartLocation.GetValue(Owner->EmitterTime, Context.GetDistributionData(), 0, InRandomStream);
	}

	LocationOffset = Owner->EmitterToSimulation.TransformVector(LocationOffset);
	Particle.Location += LocationOffset;
	ensureMsgf(!Particle.Location.ContainsNaN(), TEXT("NaN in Particle Location. Template: %s, Component: %s"), *Context.GetTemplateName(), *Context.GetInstanceName());
}

void UParticleModuleLocation::Render3DPreview(const FPreviewContext& Context)
{
#if WITH_EDITOR
	FParticleEmitterInstance* Owner = &Context.Owner;
	check(Owner);

	FPrimitiveDrawInterface* PDI = Context.PDI;

	// Draw the location as a wire star
	FVector Position(0.0f);

	FMatrix LocalToWorld = Owner->EmitterToSimulation * Owner->SimulationToWorld;

	if (StartLocation.Distribution)
	{
		// Nothing else to do if it is constant...
		if (StartLocation.Distribution->IsA(UDistributionVectorUniform::StaticClass()))
		{
			// Draw a box showing the min/max extents
			UDistributionVectorUniform* Uniform = CastChecked<UDistributionVectorUniform>(StartLocation.Distribution);
			
			Position = (Uniform->GetMaxValue() + Uniform->GetMinValue()) / 2.0f;

			FVector MinValue = Uniform->GetMinValue();
			FVector MaxValue = Uniform->GetMaxValue();
			FVector Extent = (MaxValue - MinValue) / 2.0f;
			FVector Offset = (MaxValue + MinValue) / 2.0f;
			// We just want to rotate the offset
			Offset = LocalToWorld.TransformVector(Offset);
			DrawOrientedWireBox(PDI, LocalToWorld.GetOrigin() + Offset, 
				LocalToWorld.GetScaledAxis( EAxis::X ), LocalToWorld.GetScaledAxis( EAxis::Y ), LocalToWorld.GetScaledAxis( EAxis::Z ), 
				Extent, ModuleEditorColor, SDPG_World);
		}
		else if (StartLocation.Distribution->IsA(UDistributionVectorConstantCurve::StaticClass()))
		{
			// Draw a box showing the min/max extents
			UDistributionVectorConstantCurve* Curve = CastChecked<UDistributionVectorConstantCurve>(StartLocation.Distribution);

			//Curve->
			Position = StartLocation.GetValue(0.0f, Context.GetDistributionData());
		}
		else if (StartLocation.Distribution->IsA(UDistributionVectorConstant::StaticClass()))
		{
			Position = StartLocation.GetValue(0.0f, Context.GetDistributionData());
		}
	}

	Position = LocalToWorld.TransformPosition(Position);
	DrawWireStar(PDI, Position, 10.0f, ModuleEditorColor, SDPG_World);
#endif	//#if WITH_EDITOR
}

/*-----------------------------------------------------------------------------
	UParticleModuleLocation_Seeded implementation.
-----------------------------------------------------------------------------*/
UParticleModuleLocation_Seeded::UParticleModuleLocation_Seeded(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bSpawnModule = true;
	bSupportsRandomSeed = true;
	bRequiresLoopingNotification = true;
}

void UParticleModuleLocation_Seeded::EmitterLoopingNotify(FParticleEmitterInstance* Owner)
{
	if (RandomSeedInfo.bResetSeedOnEmitterLooping == true)
	{
		FParticleRandomSeedInstancePayload* Payload = Owner->GetModuleRandomSeedInstanceData(this);
		PrepRandomSeedInstancePayload(Owner, Payload, RandomSeedInfo);
	}
}

/*-----------------------------------------------------------------------------
	UParticleModuleLocationWorldOffset implementation.
-----------------------------------------------------------------------------*/
UParticleModuleLocationWorldOffset::UParticleModuleLocationWorldOffset(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

void UParticleModuleLocationWorldOffset::SpawnEx(const FSpawnContext& Context, struct FRandomStream* InRandomStream)
{
	SPAWN_INIT;
	FParticleEmitterInstance* Owner = &Context.Owner;
	UParticleLODLevel* LODLevel = Owner->SpriteTemplate->GetCurrentLODLevel(Owner);
	check(LODLevel);
	if (LODLevel->RequiredModule->bUseLocalSpace == false)
	{
		// Nothing to do here... the distribution value is already being in world space
		Particle.Location += StartLocation.GetValue(Owner->EmitterTime, Context.GetDistributionData(), 0, InRandomStream);
	}
	else
	{
		// We need to inverse transform the location so that the bUseLocalSpace transform uses the proper value
		FMatrix InvMat = Context.GetTransform().ToMatrixWithScale().InverseFast();
		FVector StartLoc = StartLocation.GetValue(Owner->EmitterTime, Context.GetDistributionData(), 0, InRandomStream);
		Particle.Location += InvMat.TransformVector(StartLoc);
	}
	ensureMsgf(!Particle.Location.ContainsNaN(), TEXT("NaN in Particle Location. Template: %s, Component: %s"), *Context.GetTemplateName(), *Context.GetInstanceName());
}

/*-----------------------------------------------------------------------------
	UParticleModuleLocationWorldOffset_Seeded implementation.
-----------------------------------------------------------------------------*/
UParticleModuleLocationWorldOffset_Seeded::UParticleModuleLocationWorldOffset_Seeded(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bSpawnModule = true;
	bSupportsRandomSeed = true;
	bRequiresLoopingNotification = true;
}

void UParticleModuleLocationWorldOffset_Seeded::EmitterLoopingNotify(FParticleEmitterInstance* Owner)
{
	if (RandomSeedInfo.bResetSeedOnEmitterLooping == true)
	{
		FParticleRandomSeedInstancePayload* Payload = Owner->GetModuleRandomSeedInstanceData(this);
		PrepRandomSeedInstancePayload(Owner, Payload, RandomSeedInfo);
	}
}

/*-----------------------------------------------------------------------------
	UParticleModuleLocationDirect implementation.
-----------------------------------------------------------------------------*/
UParticleModuleLocationDirect::UParticleModuleLocationDirect(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bSpawnModule = true;
	bUpdateModule = true;
}

void UParticleModuleLocationDirect::InitializeDefaults()
{
	if (!Location.IsCreated())
	{
		Location.Distribution = NewObject<UDistributionVectorUniform>(this, TEXT("DistributionLocation"));
	}

	if (!LocationOffset.IsCreated())
	{
		UDistributionVectorConstant* DistributionLocationOffset = NewObject<UDistributionVectorConstant>(this, TEXT("DistributionLocationOffset"));
		DistributionLocationOffset->Constant = FVector(0.0f, 0.0f, 0.0f);
		LocationOffset.Distribution = DistributionLocationOffset;
	}

	if (!Direction.IsCreated())
	{
		UDistributionVectorConstant* DistributionScaleFactor = NewObject<UDistributionVectorConstant>(this, TEXT("DistributionScaleFactor"));
		DistributionScaleFactor->Constant = FVector(1.0f, 1.0f, 1.0f);
		ScaleFactor.Distribution = DistributionScaleFactor;

		Direction.Distribution = NewObject<UDistributionVectorUniform>(this, TEXT("DistributionDirection"));
	}
}

void UParticleModuleLocationDirect::PostInitProperties()
{
	Super::PostInitProperties();
	if (!HasAnyFlags(RF_ClassDefaultObject | RF_NeedLoad))
	{
		InitializeDefaults();
	}
}

#if WITH_EDITOR
void UParticleModuleLocationDirect::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	InitializeDefaults();
	Super::PostEditChangeProperty(PropertyChangedEvent);
}
#endif // WITH_EDITOR

void UParticleModuleLocationDirect::Spawn(const FSpawnContext& Context)
{
	SPAWN_INIT;
	FParticleEmitterInstance* Owner = &Context.Owner;
	UParticleLODLevel* LODLevel	= Owner->SpriteTemplate->GetCurrentLODLevel(Owner);
	check(LODLevel);
	if (LODLevel->RequiredModule->bUseLocalSpace)
	{
		Particle.Location = Location.GetValue(Particle.RelativeTime, Context.GetDistributionData());
	}
	else
	{
		FVector StartLoc	= Location.GetValue(Particle.RelativeTime, Context.GetDistributionData());
		StartLoc = Context.GetTransform().TransformPosition(StartLoc);
		Particle.Location	= StartLoc;
	}

	PARTICLE_ELEMENT(FVector, LocOffset);
	LocOffset	= LocationOffset.GetValue(Owner->EmitterTime, Context.GetDistributionData());
	Particle.Location += LocOffset;
	ensureMsgf(!Particle.Location.ContainsNaN(), TEXT("NaN in Particle Location. Template: %s, Component: %s"), *Context.GetTemplateName(), *Context.GetInstanceName());
}

void UParticleModuleLocationDirect::Update(const FUpdateContext& Context)
{
	FParticleEmitterInstance* Owner = &Context.Owner;
	const FTransform& OwnerTM = Owner->Component.GetAsyncComponentToWorld();
	BEGIN_UPDATE_LOOP;
	{
		FVector	NewLoc;
		UParticleLODLevel* LODLevel = Owner->SpriteTemplate->GetCurrentLODLevel(Owner);
		check(LODLevel);
		if (LODLevel->RequiredModule->bUseLocalSpace)
		{
			NewLoc = Location.GetValue(Particle.RelativeTime, Context.GetDistributionData());
		}
		else
		{
			FVector Loc			= Location.GetValue(Particle.RelativeTime, Context.GetDistributionData());
			Loc = OwnerTM.TransformPosition(Loc);
			NewLoc	= Loc;
		}

		FVector	Scale	= ScaleFactor.GetValue(Particle.RelativeTime, Context.GetDistributionData());

		PARTICLE_ELEMENT(FVector, LocOffset);
		NewLoc += LocOffset;

		FVector	Diff		 = (NewLoc - Particle.Location);
		FVector	ScaleDiffA	 = Diff * Scale.X;
		FVector	ScaleDiffB	 = Diff * (1.0f - Scale.X);
		float InvDeltaTime = (DeltaTime > 0.0f) ? 1.0f / DeltaTime : 0.0f;
		Particle.Velocity	 = (FVector3f)ScaleDiffA * InvDeltaTime;
		Particle.Location	+= ScaleDiffB;
		ensureMsgf(!Particle.Location.ContainsNaN(), TEXT("NaN in Particle Location. Template: %s, Component: %s"), *Context.GetTemplateName(), *Context.GetInstanceName());
	}
	END_UPDATE_LOOP;
}

uint32 UParticleModuleLocationDirect::RequiredBytes(UParticleModuleTypeDataBase* TypeData)
{
	return sizeof(FVector);
}

/*-----------------------------------------------------------------------------
	UParticleModuleLocationEmitter implementation.
-----------------------------------------------------------------------------*/
UParticleModuleLocationEmitter::UParticleModuleLocationEmitter(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	// Structure to hold one-time initialization
	struct FConstructorStatics
	{
		FName NAME_None;
		FConstructorStatics()
			: NAME_None(TEXT("None"))
		{
		}
	};
	static FConstructorStatics ConstructorStatics;

	bSpawnModule = true;
	SelectionMethod = ELESM_Random;
	EmitterName = ConstructorStatics.NAME_None;
	InheritSourceVelocity = false;
	InheritSourceVelocityScale = 1.0f;
	bInheritSourceRotation = false;
	InheritSourceRotationScale = 1.0f;
}

void UParticleModuleLocationEmitter::Spawn(const FSpawnContext& Context)
{
	FParticleEmitterInstance* Owner = &Context.Owner;

	// We need to look up the emitter instance...
	// This may not need to be done every Spawn, but in the short term, it will to be safe.
	// (In the editor, the source emitter may be deleted, etc...)
	FParticleEmitterInstance* LocationEmitterInst = NULL;
	if (EmitterName != NAME_None)
	{
		TArrayView<FParticleEmitterInstance*> EmitterInstances = Owner->Component.GetEmitterInstances();
		for (int32 ii = 0; ii < EmitterInstances.Num(); ii++)
		{
			FParticleEmitterInstance* pkEmitInst = EmitterInstances[ii];
			if (pkEmitInst && (pkEmitInst->SpriteTemplate->EmitterName == EmitterName))
			{
				LocationEmitterInst = pkEmitInst;
				break;
			}
		}
	}

	if (LocationEmitterInst == NULL)
	{
		// No source emitter, so we don't spawn??
		return;
	}

	check(LocationEmitterInst->CurrentLODLevel);
	check(LocationEmitterInst->CurrentLODLevel->RequiredModule);
	check(Owner->CurrentLODLevel);
	check(Owner->CurrentLODLevel->RequiredModule);
	bool bSourceIsInLocalSpace = LocationEmitterInst->CurrentLODLevel->RequiredModule->bUseLocalSpace;
	bool bInLocalSpace = Owner->CurrentLODLevel->RequiredModule->bUseLocalSpace;

	FRandomStream& RandomStream = GetRandomStream(Context);

	SPAWN_INIT;
		{
			int32 Index = 0;

			switch (SelectionMethod)
			{
			case ELESM_Random:
				{
					Index = RandomStream.RandHelper(LocationEmitterInst->ActiveParticles);
				}
				break;
			case ELESM_Sequential:
				{
					FLocationEmitterInstancePayload* Payload = 
						(FLocationEmitterInstancePayload*)(Owner->GetModuleInstanceData(this));
					if (Payload != NULL)
					{
						Index = ++(Payload->LastSelectedIndex);
						if (Index >= LocationEmitterInst->ActiveParticles)
						{
							Index = 0;
							Payload->LastSelectedIndex = Index;
						}
					}
					else
					{
						// There was an error...
						//@todo.SAS. How to resolve this situation??
					}
				}
				break;
			}
					
			// Grab a particle from the location emitter instance
			FBaseParticle* pkParticle = LocationEmitterInst->GetParticle(Index);
			if (pkParticle)
			{
				if ((pkParticle->RelativeTime == 0.0f) && (pkParticle->Location == FVector::ZeroVector))
				{
					if (bInLocalSpace == false)
					{
						Particle.Location = LocationEmitterInst->Component.GetComponentTransform().GetLocation();
					}
					else
					{
						Particle.Location = FVector::ZeroVector;
					}
				}
				else
				{
					if (bSourceIsInLocalSpace == bInLocalSpace)
					{
						// Just copy it directly
						Particle.Location = pkParticle->Location;
					}
					else if ((bSourceIsInLocalSpace == true) && (bInLocalSpace == false))
					{
						// We need to transform it into world space
						Particle.Location = LocationEmitterInst->Component.GetComponentTransform().TransformPosition(pkParticle->Location);
					}
					else //if ((bSourceIsInLocalSpace == false) && (bInLocalSpace == true))
					{
						// We need to transform it into local space
						Particle.Location = LocationEmitterInst->Component.GetComponentTransform().InverseTransformPosition(pkParticle->Location);
					}
				}
				if (InheritSourceVelocity)
				{
					Particle.BaseVelocity	+= pkParticle->Velocity * InheritSourceVelocityScale;
					Particle.Velocity		+= pkParticle->Velocity * InheritSourceVelocityScale;
				}

				if (bInheritSourceRotation)
				{
					// If the ScreenAlignment of the source emitter is PSA_Velocity, 
					// and that of the local is not, then the rotation will NOT be correct!
					Particle.Rotation		+= pkParticle->Rotation * InheritSourceRotationScale;

					// for mesh emitters only: get the mesh rotation payloads for both emitters and update the rotations accordingly; if the offset is 0, the module
					// doesn't exist, so we can't transfer rotation; if the offsets exist, the payload should never be nullptr.
					//
					const int32 MeshRotationOffset = Owner->GetMeshRotationOffset();
					const int32 SrcMeshRotationOffset = LocationEmitterInst->GetMeshRotationOffset();
					if (MeshRotationOffset && SrcMeshRotationOffset)
					{
						FMeshRotationPayloadData* DestPayloadData = (FMeshRotationPayloadData*)((uint8*)&Particle + MeshRotationOffset);
						FMeshRotationPayloadData* SrcPayloadData = (FMeshRotationPayloadData*)((uint8*)pkParticle + SrcMeshRotationOffset);

						ensure(DestPayloadData != nullptr && SrcPayloadData != nullptr);

						DestPayloadData->Rotation += SrcPayloadData->Rotation;
						DestPayloadData->InitialOrientation += SrcPayloadData->InitialOrientation;
					}
				}
			}
		}
		ensureMsgf(!Particle.Location.ContainsNaN(), TEXT("NaN in Particle Location. Template: %s, Component: %s"), *Context.GetTemplateName(), *Context.GetInstanceName());
		ensureMsgf(!Particle.Velocity.ContainsNaN(), TEXT("NaN in Particle Velocity. Template: %s, Component: %s"), *Context.GetTemplateName(), *Context.GetInstanceName());
		ensureMsgf(!Particle.BaseVelocity.ContainsNaN(), TEXT("NaN in Particle Base Velocity. Template: %s, Component: %s"), *Context.GetTemplateName(), *Context.GetInstanceName());
}

uint32 UParticleModuleLocationEmitter::RequiredBytesPerInstance()
{
	return sizeof(FLocationEmitterInstancePayload);
}

/*-----------------------------------------------------------------------------
	UParticleModuleLocationEmitterDirect implementation.
-----------------------------------------------------------------------------*/
UParticleModuleLocationEmitterDirect::UParticleModuleLocationEmitterDirect(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	// Structure to hold one-time initialization
	struct FConstructorStatics
	{
		FName NAME_None;
		FConstructorStatics()
			: NAME_None(TEXT("None"))
		{
		}
	};
	static FConstructorStatics ConstructorStatics;

	bSpawnModule = true;
	bUpdateModule = true;
	EmitterName = ConstructorStatics.NAME_None;
}

void UParticleModuleLocationEmitterDirect::Spawn(const FSpawnContext& Context)
{
	FParticleEmitterInstance* Owner = &Context.Owner;

	// We need to look up the emitter instance...
	// This may not need to be done every Spawn, but in the short term, it will to be safe.
	// (In the editor, the source emitter may be deleted, etc...)
	FParticleEmitterInstance* LocationEmitterInst = NULL;
	if (EmitterName != NAME_None)
	{
		TArrayView<FParticleEmitterInstance*> EmitterInstances = Owner->Component.GetEmitterInstances();
		for (int32 ii = 0; ii < EmitterInstances.Num(); ii++)
		{
			FParticleEmitterInstance* pkEmitInst = EmitterInstances[ii];
			if (pkEmitInst && (pkEmitInst->SpriteTemplate->EmitterName == EmitterName))
			{
				LocationEmitterInst = pkEmitInst;
				break;
			}
		}
	}

	if (LocationEmitterInst == NULL)
	{
		// No source emitter, so we don't spawn??
		return;
	}

	SPAWN_INIT;
		int32 Index = Owner->ActiveParticles;

		// Grab a particle from the location emitter instance
		FBaseParticle* pkParticle = LocationEmitterInst->GetParticle(Index);
		if (pkParticle)
		{
			Particle.Location		= pkParticle->Location;
			Particle.OldLocation	= pkParticle->OldLocation;
			Particle.Velocity		= pkParticle->Velocity;
			Particle.RelativeTime	= pkParticle->RelativeTime;
			ensureMsgf(!Particle.Location.ContainsNaN(), TEXT("NaN in Particle Location. Template: %s, Component: %s"), *Context.GetTemplateName(), *Context.GetInstanceName());
			ensureMsgf(!Particle.Velocity.ContainsNaN(), TEXT("NaN in Particle Velocity. Template: %s, Component: %s"), *Context.GetTemplateName(), *Context.GetInstanceName());
		}
} 

void UParticleModuleLocationEmitterDirect::Update(const FUpdateContext& Context)
{
	FParticleEmitterInstance* Owner = &Context.Owner;

	// We need to look up the emitter instance...
	// This may not need to be done every Spawn, but in the short term, it will to be safe.
	// (In the editor, the source emitter may be deleted, etc...)
	FParticleEmitterInstance* LocationEmitterInst = NULL;
	if (EmitterName != NAME_None)
	{
		TArrayView<FParticleEmitterInstance*> EmitterInstances = Owner->Component.GetEmitterInstances();
		for (int32 ii = 0; ii < EmitterInstances.Num(); ii++)
		{
			FParticleEmitterInstance* pkEmitInst = EmitterInstances[ii];
			if (pkEmitInst && (pkEmitInst->SpriteTemplate->EmitterName == EmitterName))
			{
				LocationEmitterInst = pkEmitInst;
				break;
			}
		}
	}

	if (LocationEmitterInst == NULL)
	{
		// No source emitter, so we don't spawn??
		return;
	}

	BEGIN_UPDATE_LOOP;
		{
			// Grab a particle from the location emitter instance
			FBaseParticle* pkParticle = LocationEmitterInst->GetParticle(i);
			if (pkParticle)
			{
				Particle.Location		= pkParticle->Location;
				Particle.OldLocation	= pkParticle->OldLocation;
				Particle.Velocity		= pkParticle->Velocity;
				Particle.RelativeTime	= pkParticle->RelativeTime;
				ensureMsgf(!Particle.Location.ContainsNaN(), TEXT("NaN in Particle Location. Template: %s, Component: %s"), *Context.GetTemplateName(), *Context.GetInstanceName());
				ensureMsgf(!Particle.Velocity.ContainsNaN(), TEXT("NaN in Particle Velocity. Template: %s, Component: %s"), *Context.GetTemplateName(), *Context.GetInstanceName());
			}
		}
	END_UPDATE_LOOP;
}

/*-----------------------------------------------------------------------------
	UParticleModuleLocationPrimitiveBase implementation.
-----------------------------------------------------------------------------*/
UParticleModuleLocationPrimitiveBase::UParticleModuleLocationPrimitiveBase(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bSpawnModule = true;
	Positive_X = true;
	Positive_Y = true;
	Positive_Z = true;
	Negative_X = true;
	Negative_Y = true;
	Negative_Z = true;
	SurfaceOnly = false;
	Velocity = false;
}

void UParticleModuleLocationPrimitiveBase::InitializeDefaults()
{
	if (!VelocityScale.IsCreated())
	{
		UDistributionFloatConstant* DistributionVelocityScale = NewObject<UDistributionFloatConstant>(this, TEXT("DistributionVelocityScale"));
		DistributionVelocityScale->Constant = 1.0f;
		VelocityScale.Distribution = DistributionVelocityScale;
	}

	if (!StartLocation.IsCreated())
	{
		UDistributionVectorConstant* DistributionStartLocation = NewObject<UDistributionVectorConstant>(this, TEXT("DistributionStartLocation"));
		DistributionStartLocation->Constant = FVector(0.0f, 0.0f, 0.0f);
		StartLocation.Distribution = DistributionStartLocation;
	}
}

void UParticleModuleLocationPrimitiveBase::PostInitProperties()
{
	Super::PostInitProperties();
	if (!HasAnyFlags(RF_ClassDefaultObject | RF_NeedLoad))
	{
		InitializeDefaults();
	}
}

#if WITH_EDITOR
void UParticleModuleLocationPrimitiveBase::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	InitializeDefaults();
	Super::PostEditChangeProperty(PropertyChangedEvent);
}
#endif // WITH_EDITOR

void UParticleModuleLocationPrimitiveBase::DetermineUnitDirection(FParticleEmitterInstance* Owner, FVector& vUnitDir, struct FRandomStream* InRandomStream)
{
	FVector vRand;

	// Grab 3 random numbers for the axes
	vRand.X	= InRandomStream->GetFraction();
	vRand.Y = InRandomStream->GetFraction();
	vRand.Z = InRandomStream->GetFraction();

	// Set the unit dir
	if (Positive_X && Negative_X)
	{
		vUnitDir.X = vRand.X * 2 - 1;
	}
	else if (Positive_X)
	{
		vUnitDir.X = vRand.X;
	}
	else if (Negative_X)
	{
		vUnitDir.X = -vRand.X;
	}
	else
	{
		vUnitDir.X = 0.0f;
	}

	if (Positive_Y && Negative_Y)
	{
		vUnitDir.Y = vRand.Y * 2 - 1;
	}
	else if (Positive_Y)
	{
		vUnitDir.Y = vRand.Y;
	}
	else if (Negative_Y)
	{
		vUnitDir.Y = -vRand.Y;
	}
	else
	{
		vUnitDir.Y = 0.0f;
	}

	if (Positive_Z && Negative_Z)
	{
		vUnitDir.Z = vRand.Z * 2 - 1;
	}
	else if (Positive_Z)
	{
		vUnitDir.Z = vRand.Z;
	}
	else if (Negative_Z)
	{
		vUnitDir.Z = -vRand.Z;
	}
	else
	{
		vUnitDir.Z = 0.0f;
	}
}

/*-----------------------------------------------------------------------------
	UParticleModuleLocationPrimitiveTriangle implementation.
-----------------------------------------------------------------------------*/
UParticleModuleLocationPrimitiveTriangle::UParticleModuleLocationPrimitiveTriangle(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{

	bSupported3DDrawMode = true;
	bSpawnModule = true;
}

void UParticleModuleLocationPrimitiveTriangle::InitializeDefaults()
{
	if (!StartOffset.IsCreated())
	{
		UDistributionVectorConstant* DistributionOffset = NewObject<UDistributionVectorConstant>(this, TEXT("DistributionOffset"));
		DistributionOffset->Constant = FVector::ZeroVector;
		StartOffset.Distribution = DistributionOffset;
	}

	if (!Height.IsCreated())
	{
		UDistributionFloatConstant* DistributionHeight = NewObject<UDistributionFloatConstant>(this, TEXT("DistributionHeight"));
		DistributionHeight->Constant = 50.0f;
		Height.Distribution = DistributionHeight;
	}

	if (!Angle.IsCreated())
	{
		UDistributionFloatConstant* DistributionAngle = NewObject<UDistributionFloatConstant>(this, TEXT("DistributionAngle"));
		DistributionAngle->Constant = 90.0f;
		Angle.Distribution = DistributionAngle;
	}

	if (!Thickness.IsCreated())
	{
		UDistributionFloatConstant* DistributionThickness = NewObject<UDistributionFloatConstant>(this, TEXT("DistributionThickness"));
		DistributionThickness->Constant = 0.0f;
		Thickness.Distribution = DistributionThickness;
	}
}

void UParticleModuleLocationPrimitiveTriangle::PostInitProperties()
{
	Super::PostInitProperties();
	if (!HasAnyFlags(RF_ClassDefaultObject | RF_NeedLoad))
	{
		InitializeDefaults();
	}
}

#if WITH_EDITOR
void UParticleModuleLocationPrimitiveTriangle::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	InitializeDefaults();
	Super::PostEditChangeProperty(PropertyChangedEvent);
}
#endif // WITH_EDITOR

void UParticleModuleLocationPrimitiveTriangle::Spawn(const FSpawnContext& Context)
{
	SpawnEx(Context, &GetRandomStream(Context));
}

void UParticleModuleLocationPrimitiveTriangle::SpawnEx(const FSpawnContext& Context, struct FRandomStream* InRandomStream)
{
	SPAWN_INIT;
	FParticleEmitterInstance* Owner = &Context.Owner;
	UParticleLODLevel* LODLevel	= Owner->SpriteTemplate->GetCurrentLODLevel(Owner);
	check(LODLevel);

	FVector TriOffset = StartOffset.GetValue(Owner->EmitterTime, Context.GetDistributionData(), 0, InRandomStream);
	float TriHeight = Height.GetValue(Owner->EmitterTime, Context.GetDistributionData(), InRandomStream);
	float TriAngle = Angle.GetValue(Owner->EmitterTime, Context.GetDistributionData(), InRandomStream);
	float TriThickness = Thickness.GetValue(Owner->EmitterTime, Context.GetDistributionData(), InRandomStream);
	float BaseLength = TriHeight * FMath::Tan(0.5f * TriAngle * UE_PI / 180.0f);

	FVector Corners[3];
	Corners[0] = TriOffset + FVector(+TriHeight * 0.5f, 0.0f, 0.0f);
	Corners[1] = TriOffset + FVector(-TriHeight * 0.5f, +BaseLength, 0.0f);
	Corners[2] = TriOffset + FVector(-TriHeight * 0.5f, -BaseLength, 0.0f);

	float BarycentricCoords[3] = {0};
	float ZPos = 0.0f;

	BarycentricCoords[0] = InRandomStream->GetFraction();
	BarycentricCoords[1] = InRandomStream->GetFraction();
	BarycentricCoords[2] = InRandomStream->GetFraction();
	ZPos = InRandomStream->GetFraction();
	
	FVector LocationOffset = FVector::ZeroVector;
	float Sum = FMath::Max<float>(UE_KINDA_SMALL_NUMBER, BarycentricCoords[0] + BarycentricCoords[1] + BarycentricCoords[2]);
	for (int32 i = 0; i < 3; i++)
	{
		LocationOffset += (BarycentricCoords[i] / Sum) * Corners[i];
	}
	LocationOffset.Z = ZPos * TriThickness - 0.5f * TriThickness;
	LocationOffset = Owner->EmitterToSimulation.TransformVector(LocationOffset);

	Particle.Location += LocationOffset;
	ensureMsgf(!Particle.Location.ContainsNaN(), TEXT("NaN in Particle Location. Template: %s, Component: %s"), *Context.GetTemplateName(), *Context.GetInstanceName());
}

void UParticleModuleLocationPrimitiveTriangle::Render3DPreview(const FPreviewContext& Context)
{
#if WITH_EDITOR
	FMatrix LocalToWorld = FMatrix::Identity;
	FParticleEmitterInstance* Owner = &Context.Owner;
	if (Owner != NULL)
	{
		LocalToWorld = Owner->EmitterToSimulation * Owner->SimulationToWorld;
	}

	if (StartOffset.Distribution && Height.Distribution && Angle.Distribution && Thickness.Distribution)
	{
		FVector TriOffset = StartOffset.GetValue(0.0f, NULL, 0, NULL);
		float TriHeight = Height.GetValue(0.0f, NULL, NULL);
		float TriAngle = Angle.GetValue(0.0f, NULL, NULL);
		float TriThickness = Thickness.GetValue(0.0f, NULL, NULL);
		float BaseLength = TriHeight * FMath::Tan(0.5f * TriAngle * UE_PI / 180.0f);

		FVector Corners[3];
		Corners[0] = TriOffset + FVector(+TriHeight * 0.5f, 0.0f, 0.0f);
		Corners[1] = TriOffset + FVector(-TriHeight * 0.5f, +BaseLength, 0.0f);
		Corners[2] = TriOffset + FVector(-TriHeight * 0.5f, -BaseLength, 0.0f);
		
		for (int32 i = 0; i < 3; ++i)
		{
			Corners[i] = LocalToWorld.TransformPosition(Corners[i]);
		}
		FVector ThicknessDir(0.0f, 0.0f, 0.5f * TriThickness);
		ThicknessDir = LocalToWorld.TransformVector(ThicknessDir);

		FPrimitiveDrawInterface* PDI = Context.PDI;

		FVector CenterPos = Corners[0] / 3.0f + Corners[1] / 3.0f + Corners[2] / 3.0f;
		DrawWireStar(PDI, CenterPos, 10.0f, ModuleEditorColor, SDPG_World);	

		for (int32 i = 0; i < 3; ++i)
		{
			PDI->DrawLine(
				Corners[i] + ThicknessDir,
				Corners[(i+1)%3] + ThicknessDir,
				ModuleEditorColor,
				SDPG_World
				);
			PDI->DrawLine(
				Corners[i] - ThicknessDir,
				Corners[(i+1)%3] - ThicknessDir,
				ModuleEditorColor,
				SDPG_World
				);
			PDI->DrawLine(
				Corners[i] + ThicknessDir,
				Corners[i] - ThicknessDir,
				ModuleEditorColor,
				SDPG_World
				);
		}
	}
#endif	//#if WITH_EDITOR
}

/*-----------------------------------------------------------------------------
	UParticleModuleLocationPrimitiveCylinder implementation.
-----------------------------------------------------------------------------*/
UParticleModuleLocationPrimitiveCylinder::UParticleModuleLocationPrimitiveCylinder(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	RadialVelocity = true;
	bSupported3DDrawMode = true;
	HeightAxis = PMLPC_HEIGHTAXIS_Z;
}

void UParticleModuleLocationPrimitiveCylinder::InitializeDefaults()
{
	if (!StartRadius.IsCreated())
	{
		UDistributionFloatConstant* DistributionStartRadius = NewObject<UDistributionFloatConstant>(this, TEXT("DistributionStartRadius"));
		DistributionStartRadius->Constant = 50.0f;
		StartRadius.Distribution = DistributionStartRadius;
	}

	if (!StartHeight.IsCreated())
	{
		UDistributionFloatConstant* DistributionStartHeight = NewObject<UDistributionFloatConstant>(this, TEXT("DistributionStartHeight"));
		DistributionStartHeight->Constant = 50.0f;
		StartHeight.Distribution = DistributionStartHeight;
	}
}

void UParticleModuleLocationPrimitiveCylinder::PostInitProperties()
{
	Super::PostInitProperties();
	if (!HasAnyFlags(RF_ClassDefaultObject | RF_NeedLoad))
	{
		InitializeDefaults();
	}
}

#if WITH_EDITOR
void UParticleModuleLocationPrimitiveCylinder::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	InitializeDefaults();
	Super::PostEditChangeProperty(PropertyChangedEvent);
}
#endif // WITH_EDITOR

void UParticleModuleLocationPrimitiveCylinder::Spawn(const FSpawnContext& Context)
{
	SpawnEx(Context, &GetRandomStream(Context));
}

void UParticleModuleLocationPrimitiveCylinder::SpawnEx(const FSpawnContext& Context, struct FRandomStream* InRandomStream)
{
	SPAWN_INIT;

	int32	RadialIndex0	= 0;	//X
	int32	RadialIndex1	= 1;	//Y
	int32	HeightIndex		= 2;	//Z

	switch (HeightAxis)
	{
	case PMLPC_HEIGHTAXIS_X:
		RadialIndex0	= 1;	//Y
		RadialIndex1	= 2;	//Z
		HeightIndex		= 0;	//X
		break;
	case PMLPC_HEIGHTAXIS_Y:
		RadialIndex0	= 0;	//X
		RadialIndex1	= 2;	//Z
		HeightIndex		= 1;	//Y
		break;
	case PMLPC_HEIGHTAXIS_Z:
		break;
	}

	FParticleEmitterInstance* Owner = &Context.Owner;

	// Determine the start location for the sphere
	FVector vStartLoc = StartLocation.GetValue(Owner->EmitterTime, Context.GetDistributionData(), 0, InRandomStream);

	FVector vOffset(0.0f);
	float	fStartRadius	= StartRadius.GetValue(Owner->EmitterTime, Context.GetDistributionData(), InRandomStream);
	float	fStartHeight	= StartHeight.GetValue(Owner->EmitterTime, Context.GetDistributionData(), InRandomStream) / 2.0f;


	// Determine the unit direction
	FVector vUnitDir = FVector::ZeroVector;

	bool bFoundValidValue = false;
	int32 NumberOfAttempts = 0;
	float RadiusSquared = fStartRadius * fStartRadius;
	while (!bFoundValidValue)
	{
		FVector vUnitDirTemp;
		DetermineUnitDirection(Owner, vUnitDirTemp, InRandomStream);
		vUnitDir[RadialIndex0]	= vUnitDirTemp[RadialIndex0];
		vUnitDir[RadialIndex1]	= vUnitDirTemp[RadialIndex1];
		vUnitDir[HeightIndex]	= vUnitDirTemp[HeightIndex];

		FVector2D CheckVal(vUnitDir[RadialIndex0] * fStartRadius, vUnitDir[RadialIndex1] * fStartRadius);
		if (CheckVal.SizeSquared() <= RadiusSquared)
		{
			bFoundValidValue = true;
		}
		else if (NumberOfAttempts >= 50)
		{
			// Just pass the value thru. 
			// It will clamp to the 'circle' but we tried...
			bFoundValidValue = true;
		}
		NumberOfAttempts++;
	}

	FVector vNormalizedDir = vUnitDir;
	vNormalizedDir.Normalize();

	FVector2D vUnitDir2D(vUnitDir[RadialIndex0], vUnitDir[RadialIndex1]);
	FVector2D vNormalizedDir2D = vUnitDir2D.GetSafeNormal();

	// Determine the position
	// Always want Z in the [-Height, Height] range
	vOffset[HeightIndex] = vUnitDir[HeightIndex] * fStartHeight;

	vNormalizedDir[RadialIndex0] = vNormalizedDir2D.X;
	vNormalizedDir[RadialIndex1] = vNormalizedDir2D.Y;

	if (SurfaceOnly)
	{
		// Clamp the X,Y to the outer edge...
		if (FMath::IsNearlyZero(FMath::Abs(vOffset[HeightIndex]) - fStartHeight))
		{
			// On the caps, it can be anywhere within the 'circle'
			vOffset[RadialIndex0] = vUnitDir[RadialIndex0] * fStartRadius;
			vOffset[RadialIndex1] = vUnitDir[RadialIndex1] * fStartRadius;
		}
		else
		{
			// On the sides, it must be on the 'circle'
			vOffset[RadialIndex0] = vNormalizedDir[RadialIndex0] * fStartRadius;
			vOffset[RadialIndex1] = vNormalizedDir[RadialIndex1] * fStartRadius;
		}
	}
	else
	{
		vOffset[RadialIndex0] = vUnitDir[RadialIndex0] * fStartRadius;
		vOffset[RadialIndex1] = vUnitDir[RadialIndex1] * fStartRadius;
	}

	// Clamp to the radius...
	FVector	vMax;

	vMax[RadialIndex0]	= FMath::Abs(vNormalizedDir[RadialIndex0]) * fStartRadius;
	vMax[RadialIndex1]	= FMath::Abs(vNormalizedDir[RadialIndex1]) * fStartRadius;
	vMax[HeightIndex]	= fStartHeight;

	vOffset[RadialIndex0]	= FMath::Clamp<float>(vOffset[RadialIndex0], -vMax[RadialIndex0], vMax[RadialIndex0]);
	vOffset[RadialIndex1]	= FMath::Clamp<float>(vOffset[RadialIndex1], -vMax[RadialIndex1], vMax[RadialIndex1]);
	vOffset[HeightIndex]	= FMath::Clamp<float>(vOffset[HeightIndex],  -vMax[HeightIndex],  vMax[HeightIndex]);

	// Add in the start location
	vOffset[RadialIndex0]	+= vStartLoc[RadialIndex0];
	vOffset[RadialIndex1]	+= vStartLoc[RadialIndex1];
	vOffset[HeightIndex]	+= vStartLoc[HeightIndex];

	Particle.Location += Owner->EmitterToSimulation.TransformVector(vOffset);

	if (Velocity)
	{
		FVector vVelocity;
		vVelocity[RadialIndex0]	= vOffset[RadialIndex0]	- vStartLoc[RadialIndex0];
		vVelocity[RadialIndex1]	= vOffset[RadialIndex1]	- vStartLoc[RadialIndex1];
		vVelocity[HeightIndex]	= vOffset[HeightIndex]	- vStartLoc[HeightIndex];

		if (RadialVelocity)
		{
			vVelocity[HeightIndex]	= 0.0f;
		}
		vVelocity	*= VelocityScale.GetValue(Owner->EmitterTime, Context.GetDistributionData(), InRandomStream);
		vVelocity = Owner->EmitterToSimulation.TransformVector(vVelocity);

		Particle.Velocity		+= (FVector3f)vVelocity;
		Particle.BaseVelocity	+= (FVector3f)vVelocity;
	}
	ensureMsgf(!Particle.Location.ContainsNaN(), TEXT("NaN in Particle Location. Template: %s, Component: %s"), *Context.GetTemplateName(), *Context.GetInstanceName());
	ensureMsgf(!Particle.Velocity.ContainsNaN(), TEXT("NaN in Particle Velocity. Template: %s, Component: %s"), *Context.GetTemplateName(), *Context.GetInstanceName());
}

void UParticleModuleLocationPrimitiveCylinder::Render3DPreview(const FPreviewContext& Context)
{
#if WITH_EDITOR
	// Draw the location as a wire star
	FVector Position = FVector::ZeroVector;
	FVector OwnerScale = FVector(1.0f);
	FMatrix LocalToWorld = FMatrix::Identity;
	FParticleEmitterInstance* Owner = &Context.Owner;
	if (Owner != NULL)
	{
		LocalToWorld = Owner->EmitterToSimulation * Owner->SimulationToWorld;
		OwnerScale = LocalToWorld.GetScaleVector();
	}

	FPrimitiveDrawInterface* PDI = Context.PDI;

	Position = LocalToWorld.TransformPosition(Position);
	DrawWireStar(PDI, Position, 10.0f, ModuleEditorColor, SDPG_World);

	if (StartLocation.Distribution)
	{
		if (StartLocation.Distribution->IsA(UDistributionVectorConstant::StaticClass()))
		{
			UDistributionVectorConstant* pkConstant = CastChecked<UDistributionVectorConstant>(StartLocation.Distribution);
			Position = pkConstant->Constant;
		}
		else
		if (StartLocation.Distribution->IsA(UDistributionVectorUniform::StaticClass()))
		{
			// Draw at the avg. of the min/max extents
			UDistributionVectorUniform* pkUniform = CastChecked<UDistributionVectorUniform>(StartLocation.Distribution);
			Position = (pkUniform->GetMaxValue() + pkUniform->GetMinValue()) / 2.0f;
		}
		else
		if (StartLocation.Distribution->IsA(UDistributionVectorConstantCurve::StaticClass()))
		{
			// Draw at the avg. of the min/max extents
			UDistributionVectorConstantCurve* pkCurve = CastChecked<UDistributionVectorConstantCurve>(StartLocation.Distribution);

			if (Owner != NULL)
			{
				//pkCurve->
				Position = StartLocation.GetValue(0.0f, Context.GetDistributionData());
			}
		}
	}

	// Draw a wire start at the center position
	Position = LocalToWorld.TransformPosition(Position);
	DrawWireStar(PDI,Position, 10.0f, ModuleEditorColor, SDPG_World);

	float fStartRadius = 1.0f;
	float fStartHeight = 1.0f;
	if(Owner)
	{
		fStartRadius = StartRadius.GetValue(Owner->EmitterTime, Context.GetDistributionData());
		fStartHeight = StartHeight.GetValue(Owner->EmitterTime, Context.GetDistributionData()) / 2.0f;
	}

	FVector	TransformedAxis[3];
	FVector	Axis[3];

	TransformedAxis[0] = LocalToWorld.TransformVector(FVector(1.0f, 0.0f, 0.0f)).GetSafeNormal();
	TransformedAxis[1] = LocalToWorld.TransformVector(FVector(0.0f, 1.0f, 0.0f)).GetSafeNormal();
	TransformedAxis[2] = LocalToWorld.TransformVector(FVector(0.0f, 0.0f, 1.0f)).GetSafeNormal();

	switch (HeightAxis)
	{
	case PMLPC_HEIGHTAXIS_X:
		Axis[0]	= TransformedAxis[1];	//Y
		Axis[1]	= TransformedAxis[2];	//Z
		Axis[2]	= TransformedAxis[0];	//X
		fStartHeight *= OwnerScale.X;
		fStartRadius *= FMath::Max<float>(OwnerScale.Y, OwnerScale.Z);
		break;
	case PMLPC_HEIGHTAXIS_Y:
		Axis[0]	= TransformedAxis[0];	//X
		Axis[1]	= TransformedAxis[2];	//Z
		Axis[2]	= TransformedAxis[1];	//Y
		fStartHeight *= OwnerScale.Y;
		fStartRadius *= FMath::Max<float>(OwnerScale.X, OwnerScale.Z);
		break;
	case PMLPC_HEIGHTAXIS_Z:
		Axis[0]	= TransformedAxis[0];	//X
		Axis[1]	= TransformedAxis[1];	//Y
		Axis[2]	= TransformedAxis[2];	//Z
		fStartHeight *= OwnerScale.Z;
		fStartRadius *= FMath::Max<float>(OwnerScale.X, OwnerScale.Y);
		break;
	}

	DrawWireCylinder(PDI,Position, Axis[0], Axis[1], Axis[2], 
		ModuleEditorColor, fStartRadius, fStartHeight, 16, SDPG_World);
#endif	//#if WITH_EDITOR
}

/*-----------------------------------------------------------------------------
	UParticleModuleLocationPrimitiveCylinder_Seeded implementation.
-----------------------------------------------------------------------------*/
UParticleModuleLocationPrimitiveCylinder_Seeded::UParticleModuleLocationPrimitiveCylinder_Seeded(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bSpawnModule = true;
	bSupportsRandomSeed = true;
	bRequiresLoopingNotification = true;
}

void UParticleModuleLocationPrimitiveCylinder_Seeded::EmitterLoopingNotify(FParticleEmitterInstance* Owner)
{
	if (RandomSeedInfo.bResetSeedOnEmitterLooping == true)
	{
		FParticleRandomSeedInstancePayload* Payload = Owner->GetModuleRandomSeedInstanceData(this);
		PrepRandomSeedInstancePayload(Owner, Payload, RandomSeedInfo);
	}
}

/*-----------------------------------------------------------------------------
	UParticleModuleLocationPrimitiveSphere implementation.
-----------------------------------------------------------------------------*/
UParticleModuleLocationPrimitiveSphere::UParticleModuleLocationPrimitiveSphere(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{

	bSupported3DDrawMode = true;
}

void UParticleModuleLocationPrimitiveSphere::InitializeDefaults()
{
	if (!StartRadius.IsCreated())
	{
		UDistributionFloatConstant* DistributionStartRadius = NewObject<UDistributionFloatConstant>(this, TEXT("DistributionStartRadius"));
		DistributionStartRadius->Constant = 50.0f;
		StartRadius.Distribution = DistributionStartRadius;
	}
}

void UParticleModuleLocationPrimitiveSphere::PostInitProperties()
{
	Super::PostInitProperties();
	if (!HasAnyFlags(RF_ClassDefaultObject | RF_NeedLoad))
	{
		InitializeDefaults();
	}
}

#if WITH_EDITOR
void UParticleModuleLocationPrimitiveSphere::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	InitializeDefaults();
	Super::PostEditChangeProperty(PropertyChangedEvent);
}
#endif // WITH_EDITOR

void UParticleModuleLocationPrimitiveSphere::Spawn(const FSpawnContext& Context)
{
	SpawnEx(Context, &GetRandomStream(Context));
}

void UParticleModuleLocationPrimitiveSphere::SpawnEx(const FSpawnContext& Context, struct FRandomStream* InRandomStream)
{
	SPAWN_INIT;
	FParticleEmitterInstance* Owner = &Context.Owner;

	// Determine the start location for the sphere
	FVector vStartLoc = StartLocation.GetValue(Owner->EmitterTime, Context.GetDistributionData(), 0, InRandomStream);

	// Determine the unit direction
	FVector vUnitDir;
	DetermineUnitDirection(Owner, vUnitDir, InRandomStream);

	FVector vNormalizedDir = vUnitDir;
	vNormalizedDir.Normalize();

	// If we want to cover just the surface of the sphere...
	if (SurfaceOnly)
	{
		vUnitDir.Normalize();
	}

	// Determine the position
	float	fStartRadius	= StartRadius.GetValue(Owner->EmitterTime, Context.GetDistributionData(), InRandomStream);
	FVector vStartRadius	= FVector(fStartRadius);
	FVector vOffset			= vUnitDir * vStartRadius;

	// Clamp to the radius...
	FVector	vMax;

	vMax.X	= FMath::Abs(vNormalizedDir.X) * fStartRadius;
	vMax.Y	= FMath::Abs(vNormalizedDir.Y) * fStartRadius;
	vMax.Z	= FMath::Abs(vNormalizedDir.Z) * fStartRadius;

	if (Positive_X || Negative_X)
	{
		vOffset.X = FMath::Clamp<float>(vOffset.X, -vMax.X, vMax.X);
	}
	else
	{
		vOffset.X = 0.0f;
	}
	if (Positive_Y || Negative_Y)
	{
		vOffset.Y = FMath::Clamp<float>(vOffset.Y, -vMax.Y, vMax.Y);
	}
	else
	{
		vOffset.Y = 0.0f;
	}
	if (Positive_Z || Negative_Z)
	{
		vOffset.Z = FMath::Clamp<float>(vOffset.Z, -vMax.Z, vMax.Z);
	}
	else
	{
		vOffset.Z = 0.0f;
	}

	vOffset += vStartLoc;
	Particle.Location += Owner->EmitterToSimulation.TransformVector(vOffset);

	if (Velocity)
	{
		FVector vVelocity		 = (vOffset - vStartLoc) * VelocityScale.GetValue(Owner->EmitterTime, Context.GetDistributionData(), InRandomStream);
		vVelocity = Owner->EmitterToSimulation.TransformVector(vVelocity);
		Particle.Velocity		+= (FVector3f)vVelocity;
		Particle.BaseVelocity	+= (FVector3f)vVelocity;
	}
	ensureMsgf(!Particle.Location.ContainsNaN(), TEXT("NaN in Particle Location. Template: %s, Component: %s"), *Context.GetTemplateName(), *Context.GetInstanceName());
	ensureMsgf(!Particle.Velocity.ContainsNaN(), TEXT("NaN in Particle Velocity. Template: %s, Component: %s"), *Context.GetTemplateName(), *Context.GetInstanceName());
}

void UParticleModuleLocationPrimitiveSphere::Render3DPreview(const FPreviewContext& Context)
{
#if WITH_EDITOR

	// Draw the location as a wire star
	FParticleEmitterInstance& Owner = Context.Owner;
	FVector Position = Owner.SimulationToWorld.TransformPosition(Owner.EmitterToSimulation.GetOrigin());

	FPrimitiveDrawInterface* PDI = Context.PDI;

	DrawWireStar(PDI, Position, 10.0f, ModuleEditorColor, SDPG_World);

	if (StartLocation.Distribution)
	{
		if (StartLocation.Distribution->IsA(UDistributionVectorConstant::StaticClass()))
		{
			UDistributionVectorConstant* pkConstant = CastChecked<UDistributionVectorConstant>(StartLocation.Distribution);
			Position = pkConstant->Constant;
		}
		else
		if (StartLocation.Distribution->IsA(UDistributionVectorUniform::StaticClass()))
		{
			// Draw at the avg. of the min/max extents
			UDistributionVectorUniform* pkUniform = CastChecked<UDistributionVectorUniform>(StartLocation.Distribution);
			Position = (pkUniform->GetMaxValue() + pkUniform->GetMinValue()) / 2.0f;
		}
		else
		if (StartLocation.Distribution->IsA(UDistributionVectorConstantCurve::StaticClass()))
		{
			// Draw at the avg. of the min/max extents
			UDistributionVectorConstantCurve* pkCurve = CastChecked<UDistributionVectorConstantCurve>(StartLocation.Distribution);

			{
				//pkCurve->
				Position = StartLocation.GetValue(0.0f, Context.GetDistributionData());
			}
		}
	}

	{
		Position = Owner.EmitterToSimulation.TransformPosition(Position);
		Position = Owner.SimulationToWorld.TransformPosition(Position);
	}

	// Draw a wire start at the center position
	DrawWireStar(PDI,Position, 10.0f, ModuleEditorColor, SDPG_World);

	float	fRadius		= 1.0f; 
	int32		iNumSides	= 32;
	FVector	vAxis[3];

	{
		fRadius = StartRadius.GetValue(Owner.EmitterTime, Context.GetDistributionData());
		vAxis[0]	= Owner.SimulationToWorld.TransformVector(Owner.EmitterToSimulation.GetScaledAxis( EAxis::X ));
		vAxis[1]	= Owner.SimulationToWorld.TransformVector(Owner.EmitterToSimulation.GetScaledAxis( EAxis::Y ));
		vAxis[2]	= Owner.SimulationToWorld.TransformVector(Owner.EmitterToSimulation.GetScaledAxis( EAxis::Z ));
	}

	DrawCircle(PDI,Position, vAxis[0], vAxis[1], ModuleEditorColor, fRadius, iNumSides, SDPG_World);
	DrawCircle(PDI,Position, vAxis[0], vAxis[2], ModuleEditorColor, fRadius, iNumSides, SDPG_World);
	DrawCircle(PDI,Position, vAxis[1], vAxis[2], ModuleEditorColor, fRadius, iNumSides, SDPG_World);
#endif	//#if WITH_EDITOR
}

/*-----------------------------------------------------------------------------
	UParticleModuleLocationPrimitiveSphere_Seeded implementation.
-----------------------------------------------------------------------------*/
UParticleModuleLocationPrimitiveSphere_Seeded::UParticleModuleLocationPrimitiveSphere_Seeded(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bSpawnModule = true;
	bSupportsRandomSeed = true;
	bRequiresLoopingNotification = true;
}

void UParticleModuleLocationPrimitiveSphere_Seeded::EmitterLoopingNotify(FParticleEmitterInstance* Owner)
{
	if (RandomSeedInfo.bResetSeedOnEmitterLooping == true)
	{
		FParticleRandomSeedInstancePayload* Payload = Owner->GetModuleRandomSeedInstanceData(this);
		PrepRandomSeedInstancePayload(Owner, Payload, RandomSeedInfo);
	}
}

/*-----------------------------------------------------------------------------
	UParticleModuleLocationBoneSocket implementation.
-----------------------------------------------------------------------------*/
UParticleModuleLocationBoneSocket::UParticleModuleLocationBoneSocket(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	// Structure to hold one-time initialization
	struct FConstructorStatics
	{
		FName NAME_BoneSocketActor;
		FConstructorStatics()
			: NAME_BoneSocketActor(TEXT("BoneSocketActor"))
		{
		}
	};
	static FConstructorStatics ConstructorStatics;

	bSpawnModule = true;
	bUpdateModule = true;
	bFinalUpdateModule = true;
	bUpdateForGPUEmitter = true;
	bSupported3DDrawMode = true;
	SourceType = BONESOCKETSOURCE_Sockets;
	SkelMeshActorParamName = ConstructorStatics.NAME_BoneSocketActor;
	bOrientMeshEmitters = true;
	SourceIndexMode = EBoneSocketSourceIndexMode::Direct;
	NumPreSelectedIndices = 10;
	InheritVelocityScale = 1.0f;
}

int32 UParticleModuleLocationBoneSocket::SelectNextSpawnIndex(FModuleLocationBoneSocketInstancePayload* InstancePayload, USkeletalMeshComponent* SourceComponent, FRandomStream& InRandomStream)
{
	check(InstancePayload && SourceComponent);

	int32 SourceIndex = -1;
	int32 MaxIndex = GetMaxSourceIndex(InstancePayload, SourceComponent);
		
	//If we're selecting from a pre generated list then always select sequentially, randomness will be introduced when generating the list.
	if (SelectionMethod == BONESOCKETSEL_Sequential || SourceIndexMode == EBoneSocketSourceIndexMode::PreSelectedIndices)
	{
		// Simply select the next socket
		SourceIndex = InstancePayload->LastSelectedIndex++;
		if (InstancePayload->LastSelectedIndex >= MaxIndex)
		{
			InstancePayload->LastSelectedIndex = 0;
		}
	}
	else if (SelectionMethod == BONESOCKETSEL_Random)
	{
		// Note: This can select the same socket over and over...
		SourceIndex = FMath::TruncToInt(InRandomStream.FRand() * ((float)MaxIndex - 0.5f));
		InstancePayload->LastSelectedIndex = SourceIndex;
	}

	if (SourceIndex == -1)
	{
		return INDEX_NONE;
	}
	if (SourceIndex >= MaxIndex)
	{
		return INDEX_NONE;
	}
		
	return SourceIndex;
}

void UParticleModuleLocationBoneSocket::RegeneratePreSelectedIndices(FModuleLocationBoneSocketInstancePayload* InstancePayload, USkeletalMeshComponent* SourceComponent, FRandomStream& InRandomStream)
{
	if (SourceIndexMode == EBoneSocketSourceIndexMode::PreSelectedIndices)
	{
		int32 MaxIndex = SourceType == BONESOCKETSOURCE_Sockets ? SourceComponent->GetSkeletalMeshAsset()->NumSockets() : SourceComponent->GetNumBones();
		for (int32 i = 0; i < NumPreSelectedIndices; ++i)
		{
			//Should we provide sequential selection here? Does that make sense for the pre selected list?
			InstancePayload->PreSelectedBoneSocketIndices[i] = FMath::TruncToInt(InRandomStream.FRand() * ((float)MaxIndex - 0.5f));
		}

		if (InheritingBoneVelocity())
		{
			//Init the bone locations so the next tick we get correct velocities.
			UpdatePrevBoneLocationsAndVelocities(InstancePayload, SourceComponent, 0.0f);
		}
	}
}

void UParticleModuleLocationBoneSocket::SetSourceIndexMode()
{
	if (SourceLocations.Num() > 0)
	{
		SourceIndexMode = EBoneSocketSourceIndexMode::SourceLocations;
	}
	else
	{
		if (InheritingBoneVelocity())
		{
			SourceIndexMode = EBoneSocketSourceIndexMode::PreSelectedIndices;
		}
		else
		{
			SourceIndexMode = EBoneSocketSourceIndexMode::Direct;
		}
	}
}

void UParticleModuleLocationBoneSocket::ValidateLODLevels(UParticleEmitter* Emitter, int32 iModule)
{
	const int32 NumLODLevels = Emitter->LODLevels.Num();
	if ( NumLODLevels <= 1 )
		return;

	bool bRequiresValidate = false;
	for (int32 iLOD=0; iLOD < NumLODLevels; ++iLOD)
	{
		UParticleLODLevel* LODLevel = Emitter->LODLevels[iLOD];
		UParticleModuleLocationBoneSocket* LODModule = CastChecked<UParticleModuleLocationBoneSocket>(LODLevel->Modules[iModule]);
		if (!LODModule->bInheritBoneVelocity && !LODModule->bUpdatePositionEachFrame)
			continue;

		bRequiresValidate = true;
	}

	if (bRequiresValidate)
	{
		UParticleModuleLocationBoneSocket* HighestLODModule = CastChecked<UParticleModuleLocationBoneSocket>(Emitter->LODLevels[0]->Modules[iModule]);

		for (int32 iLOD=1; iLOD < NumLODLevels; ++iLOD)
		{
			UParticleLODLevel* LODLevel = Emitter->LODLevels[iLOD];
			UParticleModuleLocationBoneSocket* LODModule = CastChecked<UParticleModuleLocationBoneSocket>(LODLevel->Modules[iModule]);
			if (LODModule == HighestLODModule)
				continue;

			if (LODModule->SourceLocations.Num() != HighestLODModule->SourceLocations.Num())
			{
				UE_LOGF(LogParticles, Warning, "UParticleModuleLocationBoneSocket: ParticleSystem '%ls' contains an Emitter '%ls' LOD '%d' with inconsistent SourceLocations that could lead to a crash or unexpected behaviour.  Fixing by copying the Highest LOD SourceLocations to LOD.", *Emitter->GetOuter()->GetFullName(), *Emitter->GetEmitterName().ToString(), iLOD);
				LODModule->SourceLocations = HighestLODModule->SourceLocations;
			}
		}
	}
}

void UParticleModuleLocationBoneSocket::PostLoad()
{
	Super::PostLoad();
	SetSourceIndexMode();
}

void UParticleModuleLocationBoneSocket::Spawn(const FSpawnContext& Context)
{
	FParticleEmitterInstance* Owner = &Context.Owner;

	FModuleLocationBoneSocketInstancePayload* InstancePayload = 
		(FModuleLocationBoneSocketInstancePayload*)(Owner->GetModuleInstanceData(this));
	if (InstancePayload == NULL)
	{
		return;
	}

	FRandomStream& RandomStream = GetRandomStream(Context);

	// Setup the source skeletal mesh component...
	GetSkeletalMeshComponentSource(Context, InstancePayload);

	// Early out if source component is still invalid 
	if (!InstancePayload->SourceComponent.IsValid())
	{
		return;
	}
	USkeletalMeshComponent* SourceComponent = InstancePayload->SourceComponent.Get();

	int32 SourceIndex = SelectNextSpawnIndex(InstancePayload, SourceComponent, RandomStream);
	if (SourceIndex == INDEX_NONE)
	{
		return;
	}

	FVector SourceLocation;
	FQuat RotationQuat = FQuat::Identity; // We use this later so we *must* initialize it properly.
	const int32 MeshRotationOffset = Owner->GetMeshRotationOffset();
	const bool bMeshRotationActive = MeshRotationOffset > 0 && Owner->IsMeshRotationActive(); // Note that this will *never* be false because this module always reports as touching mesh rotation
	FQuat* SourceRotation = (bMeshRotationActive) ? NULL : &RotationQuat; // We always will pass NULL down here due to the above condition.
	if (GetParticleLocation(InstancePayload, Owner, SourceComponent, SourceIndex, SourceLocation, SourceRotation) == true)
	{
		SPAWN_INIT
		{
			FModuleLocationBoneSocketParticlePayload* ParticlePayload = (FModuleLocationBoneSocketParticlePayload*)((uint8*)&Particle + Context.Offset);
			ParticlePayload->SourceIndex = SourceIndex;
			Particle.Location = SourceLocation;
			ensureMsgf(!Particle.Location.ContainsNaN(), TEXT("NaN in Particle Location. Template: %s, Component: %s"), *Context.GetTemplateName(), *Context.GetInstanceName());
			if (InheritingBoneVelocity())
			{
				// Set the base velocity for this particle.
				Particle.BaseVelocity = FMath::Lerp<FVector3f>(Particle.BaseVelocity, InstancePayload->BoneSocketVelocities[SourceIndex], InheritVelocityScale);
				ensureMsgf(!Particle.BaseVelocity.ContainsNaN(), TEXT("NaN in Particle Base Velocity. Template: %s, Component: %s"), *Context.GetTemplateName(), *Context.GetInstanceName());
			}
			if (bMeshRotationActive) // Note that right now the rotation wil *always* be Identity (see comments above)
			{
				FMeshRotationPayloadData* PayloadData = (FMeshRotationPayloadData*)((uint8*)&Particle + MeshRotationOffset);
				PayloadData->Rotation = (FVector3f)RotationQuat.Euler();
				if (Owner->CurrentLODLevel->RequiredModule->bUseLocalSpace == true)
				{
					PayloadData->Rotation = (FVector3f)Context.GetTransform().InverseTransformVectorNoScale((FVector)PayloadData->Rotation);
				}
			}
		}
	}
}

void UParticleModuleLocationBoneSocket::UpdatePrevBoneLocationsAndVelocities(FModuleLocationBoneSocketInstancePayload* InstancePayload, USkeletalMeshComponent* SourceComponent, float DeltaTime)
{
	const float InvDeltaTime = (DeltaTime > 0.0f) ? 1.0f / DeltaTime : 0.0f;

	// Calculate velocities to be used when spawning particles later this frame
	int32 MaxIndex = GetMaxSourceIndex(InstancePayload, SourceComponent);
	FMatrix WorldBoneTM;
	FVector Offset;
	for (int32 SourceIndex = 0; SourceIndex < MaxIndex; ++SourceIndex)
	{
		if (GetBoneInfoForSourceIndex(InstancePayload, SourceComponent, SourceIndex, WorldBoneTM, Offset) && SourceIndex < InstancePayload->BoneSocketVelocities.Num())
		{
			// Calculate the velocity
			const FVector3f CurrLocation = (FVector3f)WorldBoneTM.GetOrigin();	// LWC_TODO: Precision Loss
			const FVector3f Diff = CurrLocation - InstancePayload->PrevFrameBoneSocketPositions[SourceIndex];
			InstancePayload->BoneSocketVelocities[SourceIndex] = Diff * InvDeltaTime;
			InstancePayload->PrevFrameBoneSocketPositions[SourceIndex] = CurrLocation;
		}
		else
		{
			InstancePayload->BoneSocketVelocities[SourceIndex] = FVector3f::ZeroVector;
			InstancePayload->PrevFrameBoneSocketPositions[SourceIndex] = (FVector3f)SourceComponent->GetComponentLocation();
		}
	}
}

void UParticleModuleLocationBoneSocket::Update(const FUpdateContext& Context)
{
	FParticleEmitterInstance* Owner = &Context.Owner;
	FModuleLocationBoneSocketInstancePayload* InstancePayload = 
		(FModuleLocationBoneSocketInstancePayload*)(Owner->GetModuleInstanceData(this));
	if (!InstancePayload->SourceComponent.IsValid())
	{
		return;
	}

	USkeletalMeshComponent* SourceComponent = InstancePayload->SourceComponent.Get();
	if (InheritingBoneVelocity())
	{
		UpdatePrevBoneLocationsAndVelocities(InstancePayload, SourceComponent, Context.DeltaTime);
	}

	if (bUpdatePositionEachFrame == false)
	{
		return;
	}

	// Particle Data will not exist for GPU sprite emitters.
	if(Owner->ParticleData == NULL)
	{
		return;
	}

	FVector SourceLocation;

	FQuat RotationQuat = FQuat::Identity; // We use this later so we *must* initialize it properly.
	const int32 MeshRotationOffset = Owner->GetMeshRotationOffset();
	const bool bMeshRotationActive = MeshRotationOffset > 0 && Owner->IsMeshRotationActive();// Note that this will *never* be false because this module always reports as touching mesh rotation
	FQuat* SourceRotation = (bMeshRotationActive) ? NULL : &RotationQuat; // We always will pass NULL down here due to the above condition.
	const FTransform& OwnerTM = Owner->Component.GetAsyncComponentToWorld();

	//TODO: we have bone locations stored already if we're inheriting bone velocity, see if we can use those.
	BEGIN_UPDATE_LOOP;
	{
		FModuleLocationBoneSocketParticlePayload* ParticlePayload = (FModuleLocationBoneSocketParticlePayload*)((uint8*)&Particle + Offset);
		if (GetParticleLocation(InstancePayload, Owner, SourceComponent, ParticlePayload->SourceIndex, SourceLocation, SourceRotation) == true)
		{
			Particle.Location = SourceLocation;
			ensureMsgf(!Particle.Location.ContainsNaN(), TEXT("NaN in Particle Location. Template: %s, Component: %s"), *Context.GetTemplateName(), *Context.GetInstanceName());
			if (bMeshRotationActive) // Note that right now due to logic above, the rotation will always be identity
			{
				FMeshRotationPayloadData* PayloadData = (FMeshRotationPayloadData*)((uint8*)&Particle + MeshRotationOffset);
				PayloadData->Rotation = (FVector3f)RotationQuat.Euler();
				if (Owner->CurrentLODLevel->RequiredModule->bUseLocalSpace == true)
				{
					PayloadData->Rotation = (FVector3f)OwnerTM.InverseTransformVectorNoScale((FVector)PayloadData->Rotation);
				}
			}
		}
	}
	END_UPDATE_LOOP;
}

void UParticleModuleLocationBoneSocket::FinalUpdate(const FUpdateContext& Context)
{
	Super::FinalUpdate(Context);
	FParticleEmitterInstance* Owner = &Context.Owner;

	FModuleLocationBoneSocketInstancePayload* InstancePayload = 
		(FModuleLocationBoneSocketInstancePayload*)(Owner->GetModuleInstanceData(this));
	if (!InstancePayload->SourceComponent.IsValid())
	{
		//@todo. Should we setup the source skeletal mesh component here too??
		return;
	}

	USkeletalMeshComponent* SourceComponent = InstancePayload->SourceComponent.Get();

	// Particle Data will not exist for GPU sprite emitters.
	if(Owner->ParticleData)
	{
		if (SourceType == BONESOCKETSOURCE_Sockets)
		{
			bool bHaveDeadParticles = false;
			BEGIN_UPDATE_LOOP;
			{
				FModuleLocationBoneSocketParticlePayload* ParticlePayload = (FModuleLocationBoneSocketParticlePayload*)((uint8*)&Particle + Offset);
				if (SourceComponent && SourceComponent->GetSkeletalMeshAsset())
				{
					USkeletalMeshSocket* Socket;
					FVector SocketOffset;
					if (GetSocketInfoForSourceIndex(InstancePayload, SourceComponent, ParticlePayload->SourceIndex, Socket, SocketOffset))
					{
						//@todo. Can we make this faster??? Pre-find the bone index for each socket! Depending on SourceIndexMode can be done either on init or per bone, not per particle!
						int32 BoneIndex = SourceComponent->GetBoneIndex(Socket->BoneName);
						if (BoneIndex != INDEX_NONE)
						{
							if ((SourceComponent->IsBoneHidden(BoneIndex)) || 
								(SourceComponent->GetBoneTransform(BoneIndex).GetScale3D() == FVector::ZeroVector))
							{
								// Kill it
								Particle.RelativeTime = 1.1f;
								bHaveDeadParticles = true;
							}
						}
					}
				}
			}
			END_UPDATE_LOOP;
	
			if (bHaveDeadParticles == true)
			{
				Owner->KillParticles();
			}
		}
	}

	//Select a new set of bones to spawn from next frame.
	RegeneratePreSelectedIndices(InstancePayload, SourceComponent, GetRandomStream(Context));
}

uint32 UParticleModuleLocationBoneSocket::RequiredBytes(UParticleModuleTypeDataBase* TypeData)
{
	return sizeof(FModuleLocationBoneSocketParticlePayload);
}

uint32 UParticleModuleLocationBoneSocket::RequiredBytesPerInstance()
{
	// Memory in addition to the struct size is reserved for the PrevFrameBonePositions and BoneVelocity arrays. 
	// The size of these arrays are fixed to SourceLocations.Num(). FModuleLocationBoneSocketInstancePayload contains
	// an interface to access each array which are setup in PrepPerInstanceBlock to the respective offset into the instance buffer.

	SetSourceIndexMode();

	//Have to take the max of all variants as lots of code assumes all LODs use the same memory and prep it the same way :(
	int32 ArraySize = FMath::Max(SourceLocations.Num(), NumPreSelectedIndices);
	int32 ElemSize = (sizeof(FVector3f)* 2) + sizeof(int32);
	
	int32 BoneArraySize = ArraySize * ElemSize;
	
	return sizeof(FModuleLocationBoneSocketInstancePayload) + BoneArraySize;
}

uint32 UParticleModuleLocationBoneSocket::PrepPerInstanceBlock(FParticleEmitterInstance* Owner, void* InstData)
{
	FModuleLocationBoneSocketInstancePayload* Payload = (FModuleLocationBoneSocketInstancePayload*)InstData;
	if (Payload)
	{
		FMemory::Memzero(Payload, sizeof(FModuleLocationBoneSocketInstancePayload));

		int32 ArraySize = FMath::Max(SourceLocations.Num(), NumPreSelectedIndices);
	
		if (ArraySize > 0)
		{
			Payload->InitArrayProxies(ArraySize);
		}
	}
	return 0xffffffff;
}

void UParticleModuleLocationBoneSocket::AutoPopulateInstanceProperties(UParticleSystemComponent* PSysComp)
{
	check(IsInGameThread());
	bool bFound = false;
	for (int32 ParamIdx = 0; ParamIdx < PSysComp->InstanceParameters.Num(); ParamIdx++)
	{
		FParticleSysParam* Param = &(PSysComp->InstanceParameters[ParamIdx]);
		if (Param->Name == SkelMeshActorParamName)
		{
			bFound = true;
			break;
		}
	}

	if (bFound == false)
	{
		int32 NewParamIndex = PSysComp->InstanceParameters.AddZeroed();
		PSysComp->InstanceParameters[NewParamIndex].Name = SkelMeshActorParamName;
		PSysComp->InstanceParameters[NewParamIndex].ParamType = PSPT_Actor;
		PSysComp->InstanceParameters[NewParamIndex].Actor = NULL;
	}
}

#if WITH_EDITOR
void UParticleModuleLocationBoneSocket::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	FParticleResetContext ResetContext;
	ResetContext.AddTemplate(this);
}

int32 UParticleModuleLocationBoneSocket::GetNumberOfCustomMenuOptions() const
{
	return 1;
}

bool UParticleModuleLocationBoneSocket::GetCustomMenuEntryDisplayString(int32 InEntryIndex, FString& OutDisplayString) const
{
	if (InEntryIndex == 0)
	{
		OutDisplayString = NSLOCTEXT("UnrealEd", "Module_LocationBoneSocket_AutoFill", "Auto-fill Bone/Socket Names").ToString();
		return true;
	}
	return false;
}

bool UParticleModuleLocationBoneSocket::PerformCustomMenuEntry(int32 InEntryIndex)
{
	if (GIsEditor == true)
	{
		if (InEntryIndex == 0)
		{
			// Fill in the socket names array with the skeletal mesh 
			if (EditorSkelMesh != NULL)
			{
				if (SourceType == BONESOCKETSOURCE_Sockets)
				{
					const TArray<USkeletalMeshSocket*>& Sockets = EditorSkelMesh->GetActiveSocketList();

					// Retrieve all the sockets
					if (Sockets.Num() > 0)
					{
						SourceLocations.Empty();
						SourceLocations.AddZeroed( Sockets.Num() );

						for (int32 SocketIdx = 0; SocketIdx < Sockets.Num(); SocketIdx++)
						{
							FLocationBoneSocketInfo& Info = SourceLocations[SocketIdx];
							USkeletalMeshSocket* Socket = Sockets[SocketIdx];
							if (Socket != NULL)
							{
								Info.BoneSocketName = Socket->SocketName;
							}
							else
							{
								Info.BoneSocketName = NAME_None;
							}
						}
						return true;
					}
					else
					{
						FMessageDialog::Open( EAppMsgType::Ok, NSLOCTEXT("UnrealEd", "Module_LocationBoneSocket_EditorMeshNoSockets", "Editor mesh has no sockets.") );
					}
				}
				else //BONESOCKETSOURCE_Bones
				{
					// Retrieve all the bones
					if (EditorSkelMesh->GetRefSkeleton().GetNum() > 0)
					{
						SourceLocations.Empty();
						for (int32 BoneIdx = 0; BoneIdx < EditorSkelMesh->GetRefSkeleton().GetNum(); BoneIdx++)
						{
							int32 NewItemIdx = SourceLocations.AddZeroed();
							FLocationBoneSocketInfo& Info = SourceLocations[NewItemIdx];
							Info.BoneSocketName = EditorSkelMesh->GetRefSkeleton().GetBoneName(BoneIdx);
						}
						return true;
					}
					else
					{
						FMessageDialog::Open( EAppMsgType::Ok, NSLOCTEXT("UnrealEd", "Module_LocationBoneSocket_EditorMeshNoBones", "Editor mesh has no bones.") );
					}
				}
			}
			else
			{
				FMessageDialog::Open( EAppMsgType::Ok, NSLOCTEXT("UnrealEd", "Module_LocationBoneSocket_NoEditorMesh", "No editor mesh is set.") );
			}
		}
	}
	return false;
}
#endif

int32 UParticleModuleLocationBoneSocket::GetMaxSourceIndex(FModuleLocationBoneSocketInstancePayload* Payload, USkeletalMeshComponent* SourceComponent)const
{
	check(Payload);
	switch (SourceIndexMode)
	{
		case EBoneSocketSourceIndexMode::SourceLocations:
		{
			return SourceLocations.Num();
		}
		case EBoneSocketSourceIndexMode::PreSelectedIndices:
		{
			return NumPreSelectedIndices;
		}
		case EBoneSocketSourceIndexMode::Direct:
		{
			if (SourceType == BONESOCKETSOURCE_Sockets)
			{
				if (ensure(SourceComponent->GetSkeletalMeshAsset()))
				{
					return SourceComponent->GetSkeletalMeshAsset()->NumSockets();
				}
				return 0;
			}
			else
			{
				return SourceComponent->GetNumBones();
			}
		}
	}

	return 0;
}

void UParticleModuleLocationBoneSocket::GetSkeletalMeshComponentSource(const FContext& Context, FModuleLocationBoneSocketInstancePayload* InstancePayload)
{
	FParticleEmitterInstance* Owner = &Context.Owner;
	if (Owner == NULL)
	{
		InstancePayload->SourceComponent = nullptr;
		return;
	}

	UParticleSystemComponent* PSysComp = Owner->Component.AsComponent();
	if (PSysComp == NULL)
	{
		InstancePayload->SourceComponent = nullptr;
		return;
	}

	USkeletalMeshComponent* NewSkelComp = nullptr;

	AActor* Actor = nullptr;
	PSysComp->GetActorParameter(SkelMeshActorParamName, Actor);
	USkeletalMeshComponent* AttachParentMesh = Cast<USkeletalMeshComponent>(PSysComp->GetAttachParent());

	bool bActorChanged = Actor != InstancePayload->CachedActor.Get();
	bool bAttachParentChanged = (AttachParentMesh && AttachParentMesh != InstancePayload->SourceComponent.Get());

	if (!InstancePayload->SourceComponent.IsValid() || bActorChanged || bAttachParentChanged)
	{
		InstancePayload->SourceComponent = nullptr;
		InstancePayload->CachedActor = Actor;

		if (Actor)
		{
			ASkeletalMeshActor* SkelMeshActor = Cast<ASkeletalMeshActor>(Actor);
			if (SkelMeshActor != NULL)
			{
				NewSkelComp = SkelMeshActor->GetSkeletalMeshComponent();
			}
			else if (Actor)
			{
				USkeletalMeshComponent* SkeletalMeshComponent = Actor->FindComponentByClass<USkeletalMeshComponent>();
				if (SkeletalMeshComponent)
				{
					NewSkelComp = SkeletalMeshComponent;
				}
				//@todo. Warn about this...
			}
		}

		if (AttachParentMesh)
		{
			NewSkelComp = AttachParentMesh;
		}

		if (NewSkelComp)
		{
			InstancePayload->SourceComponent = NewSkelComp;
			RegeneratePreSelectedIndices(InstancePayload, NewSkelComp, GetRandomStream(Context));
		}
	}
}

bool UParticleModuleLocationBoneSocket::GetSocketInfoForSourceIndex(FModuleLocationBoneSocketInstancePayload* InstancePayload, USkeletalMeshComponent* SourceComponent, int32 SourceIndex, USkeletalMeshSocket*& OutSocket, FVector& OutOffset) const
{
	if (!ensureMsgf(SourceType == BONESOCKETSOURCE_Sockets, TEXT("Invalid source type %d for %s"), SourceType.GetIntValue(), *GetPathName()) ||
		!ensureMsgf(SourceComponent, TEXT("Null SkeletalMeshComponent for %s"), *GetPathName()) ||
		!ensureMsgf(SourceComponent->GetSkeletalMeshAsset(), TEXT("Null SkeletalMesh on Component %s for %s"), *SourceComponent->GetPathName(), *GetPathName()))
	{
		return false;
	}
	
	switch (SourceIndexMode)
	{
		case EBoneSocketSourceIndexMode::SourceLocations:
		{
			if (ensureMsgf(SourceLocations.IsValidIndex(SourceIndex), TEXT("Invalid index of %d for %s"), SourceIndex, *GetPathName()))
			{
				OutSocket = SourceComponent->GetSkeletalMeshAsset()->FindSocket(SourceLocations[SourceIndex].BoneSocketName);
				OutOffset = SourceLocations[SourceIndex].Offset + UniversalOffset;
			}
			else
			{
				return false;
			}
		}
		break;
		case EBoneSocketSourceIndexMode::PreSelectedIndices:
		{
			if (ensureMsgf(InstancePayload, TEXT("Invalid instance payload parameter on GetSocketInfoForSourceIndex %d for %s"), SourceIndex, *GetPathName()) &&
				ensureMsgf(SourceIndex >= 0 && SourceIndex < InstancePayload->PreSelectedBoneSocketIndices.Num(), TEXT("Invalid index of %d for %s"), SourceIndex, *GetPathName()))
			{
				OutSocket = SourceComponent->GetSkeletalMeshAsset()->GetSocketByIndex(InstancePayload->PreSelectedBoneSocketIndices[SourceIndex]);
				OutOffset = UniversalOffset;
			} 
			else
			{
				return false;
			}
			
		}
		break;
		case EBoneSocketSourceIndexMode::Direct:
		{
			OutSocket = SourceComponent->GetSkeletalMeshAsset()->GetSocketByIndex(SourceIndex);
			OutOffset = UniversalOffset;
		}
		break;
	}
	return OutSocket != nullptr;
}

bool UParticleModuleLocationBoneSocket::GetBoneInfoForSourceIndex(FModuleLocationBoneSocketInstancePayload* InstancePayload, USkeletalMeshComponent* SourceComponent, int32 SourceIndex, FMatrix& OutBoneMatrix, FVector& OutOffset)const
{
	int32 BoneIndex = INDEX_NONE;
	FVector Offset = FVector::ZeroVector;
	if (SourceType == BONESOCKETSOURCE_Sockets)
	{
		USkeletalMeshSocket* Socket = nullptr;
		if (GetSocketInfoForSourceIndex(InstancePayload, SourceComponent, SourceIndex, Socket, Offset))
		{
			BoneIndex = SourceComponent->GetBoneIndex(Socket->BoneName);
			return false;
		}
	}
	else
	{
		switch (SourceIndexMode)
		{
			case EBoneSocketSourceIndexMode::SourceLocations:
			{
				BoneIndex = SourceComponent->GetBoneIndex(SourceLocations[SourceIndex].BoneSocketName);
				Offset = SourceLocations[SourceIndex].Offset + UniversalOffset;
			}
			break;
			case EBoneSocketSourceIndexMode::PreSelectedIndices:
			{
				BoneIndex = InstancePayload->PreSelectedBoneSocketIndices[SourceIndex];
				Offset = UniversalOffset;
			}
			break;
			case EBoneSocketSourceIndexMode::Direct:
			{
				BoneIndex = SourceIndex;
				Offset = UniversalOffset;
			}
			break;
		}
	}

	if (BoneIndex != INDEX_NONE)
	{
		OutBoneMatrix = SourceComponent->GetBoneMatrix(BoneIndex);
		OutOffset = Offset;
		return true;
	}

	return false;
}

bool UParticleModuleLocationBoneSocket::GetParticleLocation(FModuleLocationBoneSocketInstancePayload* InstancePayload, FParticleEmitterInstance* Owner,
	USkeletalMeshComponent* InSkelMeshComponent, int32 InBoneSocketIndex, 
	FVector& OutPosition, FQuat* OutRotation)
{
	check(InSkelMeshComponent);

	if (SourceType == BONESOCKETSOURCE_Sockets)
	{
		if (InSkelMeshComponent->GetSkeletalMeshAsset())
		{
			USkeletalMeshSocket* Socket;
			FVector SocketOffset;
			if (GetSocketInfoForSourceIndex(InstancePayload, InSkelMeshComponent, InBoneSocketIndex, Socket, SocketOffset))
			{
				FRotator SocketRotator(0,0,0);
				FMatrix SocketMatrix;
				if (Socket->GetSocketMatrixWithOffset(SocketMatrix, InSkelMeshComponent, SocketOffset, SocketRotator) == false)
				{
					return false;
				}
				OutPosition = SocketMatrix.GetOrigin();
				if (OutRotation != NULL)
				{
					SocketMatrix.RemoveScaling();
					*OutRotation = SocketMatrix.ToQuat();
				}
			}
			else
			{
				return false;
			}
		}
		else
		{
			return false;
		}
	}
	else	//BONESOCKETSOURCE_Bones
	{
		FVector SocketOffset;
		FMatrix WorldBoneTM;
		if (GetBoneInfoForSourceIndex(InstancePayload, InSkelMeshComponent, InBoneSocketIndex, WorldBoneTM, SocketOffset))
		{
			FTranslationMatrix OffsetMatrix(SocketOffset);
			FMatrix ResultMatrix = OffsetMatrix * WorldBoneTM;
			OutPosition = ResultMatrix.GetOrigin();
			if (OutRotation != NULL)
			{
				ResultMatrix.RemoveScaling();
				*OutRotation = ResultMatrix.ToQuat();
			}
		}
		else
		{
			return false;
		}
	}

	if (Owner->CurrentLODLevel->RequiredModule->bUseLocalSpace == true)
	{
		OutPosition = Owner->Component.GetComponentTransform().InverseTransformPosition(OutPosition);
	}

	return true;
}

/*-----------------------------------------------------------------------------
	UParticleModuleLocationVertSurface implementation.
-----------------------------------------------------------------------------*/
UParticleModuleLocationSkelVertSurface::UParticleModuleLocationSkelVertSurface(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	// Structure to hold one-time initialization
	struct FConstructorStatics
	{
		FName NAME_VertSurfaceActor;
		FConstructorStatics()
			: NAME_VertSurfaceActor(TEXT("VertSurfaceActor"))
		{
		}
	};
	static FConstructorStatics ConstructorStatics;

	bSpawnModule = true;
	bUpdateModule = true;
	bUpdateForGPUEmitter=true;
	bFinalUpdateModule = true;
	bSupported3DDrawMode = true;
	SourceType = VERTSURFACESOURCE_Vert;
	SkelMeshActorParamName = ConstructorStatics.NAME_VertSurfaceActor;
	bOrientMeshEmitters = true;
	bEnforceNormalCheck = false;
	bInheritUV = false;
	InheritUVChannel = 0;
	InheritVelocityScale = 1.0f;
}

DEFINE_STAT(STAT_ParticleSkelMeshSurfTime);


void UParticleModuleLocationSkelVertSurface::PostLoad()
{
	Super::PostLoad();

	if(NormalCheckToleranceDegrees > 180.0f)
	{
		NormalCheckToleranceDegrees = 180.0f;
	}
	else if(NormalCheckToleranceDegrees < 0.0f)
	{
		NormalCheckToleranceDegrees = 0.0f;
	}

	NormalCheckTolerance = ((1.0f-(NormalCheckToleranceDegrees/180.0f))*2.0f)-1.0f;

	if (GetLinkerUEVersion() < VER_UE4_FIX_SKEL_VERT_ORIENT_MESH_PARTICLES)
	{
		//The code to actually do this hasn't been present ever in UE4 so I'm disabling it for old emitters.
		//I expect that some users will have this set to true and it will alter the behavior of their emitters under their feet.
		bOrientMeshEmitters = false;
	}
}

#if WITH_EDITOR
void UParticleModuleLocationSkelVertSurface::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	if(PropertyChangedEvent.Property && PropertyChangedEvent.Property->GetName() == "NormalCheckToleranceDegrees")
	{
		if(NormalCheckToleranceDegrees > 180.0f)
		{
			NormalCheckToleranceDegrees = 180.0f;
		}
		else if(NormalCheckToleranceDegrees < 0.0f)
		{
			NormalCheckToleranceDegrees = 0.0f;
		}

		NormalCheckTolerance = ((1.0f-(NormalCheckToleranceDegrees/180.0f))*2.0f)-1.0f;
	}
}
#endif // WITH_EDITOR

void UParticleModuleLocationSkelVertSurface::Spawn(const FSpawnContext& Context)
{
	FParticleEmitterInstance* Owner = &Context.Owner;
	SCOPE_CYCLE_COUNTER(STAT_ParticleSkelMeshSurfTime);
	FModuleLocationVertSurfaceInstancePayload* InstancePayload = 
		(FModuleLocationVertSurfaceInstancePayload*)(Owner->GetModuleInstanceData(this));
	if (InstancePayload == NULL)
	{
		return;
	}

	FRandomStream& RandomStream = GetRandomStream(Context);
	
	GetSkeletalMeshComponentSource(Context, InstancePayload);

	if (!InstancePayload->MeshIsValid())
	{
		SPAWN_INIT
		{
			Particle.RelativeTime = 1.1f;
		}

		return;
	}

	USkeletalMeshComponent* SourceComponent = InstancePayload->SourceComponent.Get();
	FSkeletalMeshRenderData* SkelMeshRenderData = SourceComponent ? SourceComponent->GetSkeletalMeshRenderData() : NULL;
	if (SkelMeshRenderData == NULL)
	{
		return;
	}

	FSkeletalMeshLODRenderData& LODData = SkelMeshRenderData->LODRenderData[InstancePayload->MeshMinLOD];

	// Determine the bone/socket to spawn at
	int32 SourceIndex = -1;
	int32 ActiveBoneIndex = -1;
	if (SourceType == VERTSURFACESOURCE_Vert)
	{
		int32 SourceLocationsCount(LODData.GetNumVertices());

		SourceIndex = FMath::TruncToInt(RandomStream.FRand() * ((float)SourceLocationsCount) - 1);
		InstancePayload->VertIndex = SourceIndex;

		if(SourceIndex != -1)
		{
			if(!VertInfluencedByActiveBone(Owner, InstancePayload, SourceComponent, SourceIndex, &ActiveBoneIndex))
			{
				SPAWN_INIT
				{
					Particle.RelativeTime = 1.1f;
				}

				return;
			}
		}
	}
	else if(SourceType == VERTSURFACESOURCE_Surface)
	{
		int32 SectionCount = LODData.RenderSections.Num();
		int32 RandomSection = FMath::RoundToInt(RandomStream.FRand() * ((float)SectionCount-1));

		SourceIndex = LODData.RenderSections[RandomSection].BaseIndex +
			(FMath::TruncToInt(RandomStream.FRand() * ((float)LODData.RenderSections[RandomSection].NumTriangles))*3);

		InstancePayload->VertIndex = SourceIndex;

		if(SourceIndex != -1)
		{
			int32 VertIndex[3];

			VertIndex[0] = LODData.MultiSizeIndexContainer.GetIndexBuffer()->Get( SourceIndex );
			VertIndex[1] = LODData.MultiSizeIndexContainer.GetIndexBuffer()->Get( SourceIndex+1 );
			VertIndex[2] = LODData.MultiSizeIndexContainer.GetIndexBuffer()->Get( SourceIndex+2 );

			int32 BoneIndex1, BoneIndex2, BoneIndex3;
			BoneIndex1 = BoneIndex2 = BoneIndex3 = INDEX_NONE;
			if(!VertInfluencedByActiveBone(Owner, InstancePayload, SourceComponent, VertIndex[0], &BoneIndex1) &&
			   !VertInfluencedByActiveBone(Owner, InstancePayload, SourceComponent, VertIndex[1], &BoneIndex2) &&
			   !VertInfluencedByActiveBone(Owner, InstancePayload, SourceComponent, VertIndex[2], &BoneIndex3))
			{
				SPAWN_INIT
				{
					Particle.RelativeTime = 1.1f;
				}

				return;
			}

			// Attempt to retrieve a valid bone index for any of the three verts.
			ActiveBoneIndex = FMath::Max3(BoneIndex1, BoneIndex2, BoneIndex3);
		}
	}

	if (SourceIndex == -1)
	{
		// Failed to select a vert/face?
		return;
	}

	FVector SourceLocation;
	FQuat SourceRotation;
	const int32 MeshRotationOffset = Owner->GetMeshRotationOffset();
	const bool bMeshRotationActive = MeshRotationOffset > 0 && Owner->IsMeshRotationActive();
	if (GetParticleLocation(Context, InstancePayload, SourceComponent, SourceIndex, SourceLocation, SourceRotation, true) == true)
	{
		SPAWN_INIT
		{
			FModuleLocationVertSurfaceParticlePayload* ParticlePayload = (FModuleLocationVertSurfaceParticlePayload*)((uint8*)&Particle + Context.Offset);
			ParticlePayload->SourceIndex = SourceIndex;
			Particle.Location = SourceLocation;
			ensureMsgf(!Particle.Location.ContainsNaN(), TEXT("NaN in Particle Location. Template: %s, Component: %s"), *Context.GetTemplateName(), *Context.GetInstanceName());

			// Set the base velocity
			if(bInheritBoneVelocity && ActiveBoneIndex != INDEX_NONE)
			{
				const int32 VelocityIndex = InstancePayload->ValidAssociatedBoneIndices.Find(ActiveBoneIndex);
				if(VelocityIndex != INDEX_NONE)
				{
					Particle.BaseVelocity = FMath::Lerp(Particle.BaseVelocity, InstancePayload->BoneVelocities[VelocityIndex], InheritVelocityScale);
					ensureMsgf(!Particle.BaseVelocity.ContainsNaN(), TEXT("NaN in Particle Base Velocity. Template: %s, Component: %s"), *Context.GetTemplateName(), *Context.GetInstanceName());
				}
			}
			
			if (bInheritVertexColor)
			{
				FColor UseColor;
				if (SourceType == VERTSURFACESOURCE_Vert)
				{
					UseColor = SourceComponent->GetVertexColor(SourceIndex);
				}
				else if (SourceType == VERTSURFACESOURCE_Surface)
				{
					int32 VertIndex[3];
					FColor VertColors[3];

					VertIndex[0] = LODData.MultiSizeIndexContainer.GetIndexBuffer()->Get(SourceIndex);
					VertIndex[1] = LODData.MultiSizeIndexContainer.GetIndexBuffer()->Get(SourceIndex + 1);
					VertIndex[2] = LODData.MultiSizeIndexContainer.GetIndexBuffer()->Get(SourceIndex + 2);
					VertColors[0] = SourceComponent->GetVertexColor(VertIndex[0]);
					VertColors[1] = SourceComponent->GetVertexColor(VertIndex[1]);
					VertColors[2] = SourceComponent->GetVertexColor(VertIndex[2]);
					UseColor.R = (VertColors[0].R + VertColors[1].R + VertColors[2].R) / 3;
					UseColor.G = (VertColors[0].G + VertColors[1].G + VertColors[2].G) / 3;
					UseColor.B = (VertColors[0].B + VertColors[1].B + VertColors[2].B) / 3;
					UseColor.A = (VertColors[0].A + VertColors[1].A + VertColors[2].A) / 3;
				}
				Particle.Color = UseColor;
				Particle.BaseColor = UseColor;
			}
					
			if (bInheritUV)
			{
				FVector2D UseUV = FVector2D::ZeroVector;
				if (SourceType == VERTSURFACESOURCE_Vert)
				{
					UseUV = SourceComponent->GetVertexUV(SourceIndex, InheritUVChannel);
				}
				else if (SourceType == VERTSURFACESOURCE_Surface)
				{
					int32 VertIndex[3];
					FVector2D VertUVs[3];

					VertIndex[0] = LODData.MultiSizeIndexContainer.GetIndexBuffer()->Get(SourceIndex);
					VertIndex[1] = LODData.MultiSizeIndexContainer.GetIndexBuffer()->Get(SourceIndex + 1);
					VertIndex[2] = LODData.MultiSizeIndexContainer.GetIndexBuffer()->Get(SourceIndex + 2);
					VertUVs[0] = SourceComponent->GetVertexUV(VertIndex[0], InheritUVChannel);
					VertUVs[1] = SourceComponent->GetVertexUV(VertIndex[1], InheritUVChannel);
					VertUVs[2] = SourceComponent->GetVertexUV(VertIndex[2], InheritUVChannel);
					UseUV.X = (VertUVs[0].X + VertUVs[1].X + VertUVs[2].X) / 3;
					UseUV.Y = (VertUVs[0].Y + VertUVs[1].Y + VertUVs[2].Y) / 3;

					// TODO: Barycentric interpolation instead of triangle average. Position is in same struct during vertex fetch above
					/*FVector BarycentricUV = FMath::GetBaryCentric2D(Point, VertA, VertB, VertC);
					FVector2D UseUV = VertUVs[0] * BarycentricUV.X + VertUVs[1] * BarycentricUV.Y + VertUVs[2] * BarycentricUV.Z;*/
				}

				const int32 DynParamOffset = Owner->DynamicParameterDataOffset;
				if (DynParamOffset > 0)
				{
					// Override dynamic parameters to allow vertex colors to be used
					FEmitterDynamicParameterPayload& DynamicPayload = *((FEmitterDynamicParameterPayload*)((uint8*)&Particle + DynParamOffset));
					DynamicPayload.DynamicParameterValue[0] = UseUV.X;
					DynamicPayload.DynamicParameterValue[1] = UseUV.Y;
				}
			}

			if (bMeshRotationActive)
			{
				FMeshRotationPayloadData* PayloadData = (FMeshRotationPayloadData*)((uint8*)&Particle + MeshRotationOffset);
				if (bOrientMeshEmitters)
				{
					//We have the mesh oriented to the normal of the triangle it's on but this looks fugly as particles on each triangle are facing the same way.
					//The only valid orientation reference should be the normal. So add an additional random rotation around it.
					SourceRotation = SourceRotation * FQuat(FVector::UpVector, RandomStream.FRand()*(UE_PI*2.0f));
				}

				FVector Rot = SourceRotation.Euler();
				if (Owner->CurrentLODLevel->RequiredModule->bUseLocalSpace == true)
				{
					Rot = Context.GetTransform().InverseTransformVectorNoScale(Rot);
				}
				PayloadData->Rotation = (FVector3f)Rot;
				PayloadData->InitRotation = (FVector3f)Rot;
			}
		}
	}
	else
	{
		SPAWN_INIT
		{
			Particle.RelativeTime = 1.1f;
		}
	}
}

void UParticleModuleLocationSkelVertSurface::Update(const FUpdateContext& Context)
{
	SCOPE_CYCLE_COUNTER(STAT_ParticleSkelMeshSurfTime);
	
	FParticleEmitterInstance* Owner = &Context.Owner;
	FModuleLocationVertSurfaceInstancePayload* InstancePayload = 
		(FModuleLocationVertSurfaceInstancePayload*)(Owner->GetModuleInstanceData(this));

	GetSkeletalMeshComponentSource(Context, InstancePayload);

	if (!InstancePayload->MeshIsValid())
	{
		return;
	}

	USkeletalMeshComponent* SourceComponent = InstancePayload->SourceComponent.Get();

	if(bInheritBoneVelocity)
	{
		const float InvDeltaTime = (Context.DeltaTime > 0.0f) ? 1.0f / Context.DeltaTime : 0.0f;

		// Calculate velocities to be used when spawning particles later this frame
		for(int32 ValidBoneIndex = 0; ValidBoneIndex < InstancePayload->NumValidAssociatedBoneIndices; ++ValidBoneIndex)
		{
			int32 BoneIndex = InstancePayload->ValidAssociatedBoneIndices[ValidBoneIndex];
			if (BoneIndex != INDEX_NONE)
			{
				const FMatrix WorldBoneTM = SourceComponent->GetBoneMatrix(BoneIndex);
				const FVector3f Diff = (FVector3f)WorldBoneTM.GetOrigin() - InstancePayload->PrevFrameBonePositions[ValidBoneIndex];	// LWC_TODO: Precision Loss
				InstancePayload->BoneVelocities[ValidBoneIndex] = Diff * InvDeltaTime;
			}
		}
	}

	if (bUpdatePositionEachFrame == false)
	{
		return;
	}

	// Particle Data will not exist for GPU sprite emitters.
	if(Owner->ParticleData == NULL)
	{
		return;
	}

	FVector SourceLocation;
	FQuat SourceRotation;
	const int32 MeshRotationOffset = Owner->GetMeshRotationOffset();
	const bool bMeshRotationActive = MeshRotationOffset > 0 && Owner->IsMeshRotationActive();
	const FTransform& OwnerTM = Owner->Component.GetAsyncComponentToWorld();

	FRandomStream& RandomStream = GetRandomStream(Context);

	BEGIN_UPDATE_LOOP;
	{
		FModuleLocationVertSurfaceParticlePayload* ParticlePayload = (FModuleLocationVertSurfaceParticlePayload*)((uint8*)&Particle + Offset);
		if (GetParticleLocation(Context, InstancePayload, SourceComponent, ParticlePayload->SourceIndex, SourceLocation, SourceRotation) == true)
		{
			Particle.Location = SourceLocation;
			ensureMsgf(!Particle.Location.ContainsNaN(), TEXT("NaN in Particle Location. Template: %s, Component: %s"), *Context.GetTemplateName(), *Context.GetInstanceName());
			if (bMeshRotationActive)
			{
				if (bOrientMeshEmitters)
				{
					//We have the mesh oriented to the normal of the triangle it's on but this looks fugly as particles on each triangle are facing the same way.
					//The only valid orientation reference should be the normal. So add an additional random rotation around it.
					SourceRotation = SourceRotation * FQuat(FVector::UpVector, RandomStream.FRand()*(UE_PI*2.0f));
				}

				FMeshRotationPayloadData* PayloadData = (FMeshRotationPayloadData*)((uint8*)&Particle + MeshRotationOffset);
				FVector Rot = SourceRotation.Euler();
				if (Owner->CurrentLODLevel->RequiredModule->bUseLocalSpace == true)
				{
					Rot = OwnerTM.InverseTransformVectorNoScale(Rot);
				}
				PayloadData->Rotation = (FVector3f)Rot;
			}
		}
	}
	END_UPDATE_LOOP;
}

void UParticleModuleLocationSkelVertSurface::FinalUpdate(const FUpdateContext& Context)
{
	Super::FinalUpdate(Context);

	FParticleEmitterInstance* Owner = &Context.Owner;
	FModuleLocationVertSurfaceInstancePayload* InstancePayload = 
		(FModuleLocationVertSurfaceInstancePayload*)(Owner->GetModuleInstanceData(this));
	if (!InstancePayload->MeshIsValid())
	{
		return;
	}

	USkeletalMeshComponent* SourceComponent = InstancePayload->SourceComponent.Get();

	if(bInheritBoneVelocity)
	{
		// Save bone positions to be used to calculate velocity on the next frame
		for(int32 ValidBoneIndex = 0; ValidBoneIndex < InstancePayload->NumValidAssociatedBoneIndices; ++ValidBoneIndex)
		{
		
			const int32 BoneIndex = InstancePayload->ValidAssociatedBoneIndices[ValidBoneIndex];
			if (BoneIndex != INDEX_NONE)
			{
				const FMatrix WorldBoneTM = SourceComponent->GetBoneMatrix(BoneIndex);
				InstancePayload->PrevFrameBonePositions[ValidBoneIndex] = (FVector3f)WorldBoneTM.GetOrigin();
			}
		}
	}
}

uint32 UParticleModuleLocationSkelVertSurface::PrepPerInstanceBlock(FParticleEmitterInstance* Owner, void* InstData)
{
	FModuleLocationVertSurfaceInstancePayload* Payload = (FModuleLocationVertSurfaceInstancePayload*)InstData;
	if (Payload)
	{
		Payload->InitArrayProxies(ValidAssociatedBones.Num());
	}

	return Super::PrepPerInstanceBlock(Owner, InstData);
}


void UParticleModuleLocationSkelVertSurface::UpdateBoneIndicesList(FParticleEmitterInstance* Owner)
{
	FModuleLocationVertSurfaceInstancePayload* InstancePayload = 
		(FModuleLocationVertSurfaceInstancePayload*)(Owner->GetModuleInstanceData(this));

	AActor* ActorInst = NULL;

	if (USkeletalMeshComponent* SkelMeshComp = InstancePayload->SourceComponent.Get())
	{
		int32 InsertionIndex = 0;
		for (int32 FindBoneIdx = 0; FindBoneIdx < ValidAssociatedBones.Num(); FindBoneIdx++)
		{
			const int32 BoneIdx = SkelMeshComp->GetSkeletalMeshAsset()->GetRefSkeleton().FindBoneIndex(ValidAssociatedBones[FindBoneIdx]);
			if (BoneIdx != INDEX_NONE && ValidAssociatedBones.Num() > InsertionIndex)
			{
				InstancePayload->ValidAssociatedBoneIndices[InsertionIndex++] = BoneIdx;
			}
		}
		// Cache the number of bone indices on the payload
		InstancePayload->NumValidAssociatedBoneIndices = InsertionIndex;
	}
}


uint32 UParticleModuleLocationSkelVertSurface::RequiredBytes(UParticleModuleTypeDataBase* TypeData)
{
	return sizeof(FModuleLocationVertSurfaceParticlePayload);
}


uint32 UParticleModuleLocationSkelVertSurface::RequiredBytesPerInstance()
{
	// Memory in addition to the struct size is reserved for the ValidAssociatedBoneIndices, PrevFrameBonePositions and BoneVelocity arrays. 
	// The size of these arrays are fixed to ValidAssociatedBones.Num(). Proxys are setup in PrepPerInstanceBlock 
	// to access these arrays
 
	const uint32 ArraySize = ValidAssociatedBones.Num();
	// Allocation size to reserve for ValidAssociatedBonesIndices array
	const uint32 ValidAssociatedBonesIndiceSize = ArraySize * sizeof(int32);
	// Allocation size to reserve for PrevFrameBonePositions, and BoneVelocity arrays
	const uint32 BoneArraySize = ArraySize * sizeof(FVector) * 2;
	return sizeof(FModuleLocationVertSurfaceInstancePayload) + ValidAssociatedBonesIndiceSize + BoneArraySize;
}


void UParticleModuleLocationSkelVertSurface::AutoPopulateInstanceProperties(UParticleSystemComponent* PSysComp)
{
	check(IsInGameThread());
	bool bFound = false;
	for (int32 ParamIdx = 0; ParamIdx < PSysComp->InstanceParameters.Num(); ParamIdx++)
	{
		FParticleSysParam* Param = &(PSysComp->InstanceParameters[ParamIdx]);
		if (Param->Name == SkelMeshActorParamName)
		{
			bFound = true;
			break;
		}
	}

	if (bFound == false)
	{
		int32 NewParamIndex = PSysComp->InstanceParameters.AddZeroed();
		PSysComp->InstanceParameters[NewParamIndex].Name = SkelMeshActorParamName;
		PSysComp->InstanceParameters[NewParamIndex].ParamType = PSPT_Actor;
		PSysComp->InstanceParameters[NewParamIndex].Actor = NULL;
	}
}

#if WITH_EDITOR

int32 UParticleModuleLocationSkelVertSurface::GetNumberOfCustomMenuOptions() const
{
	return 1;
}


bool UParticleModuleLocationSkelVertSurface::GetCustomMenuEntryDisplayString(int32 InEntryIndex, FString& OutDisplayString) const
{
	if (InEntryIndex == 0)
	{
		OutDisplayString = NSLOCTEXT("UnrealEd", "Module_LocationVertSurface_AutoFill", "Auto-fill Bone Names").ToString();
		return true;
	}
	return false;
}


bool UParticleModuleLocationSkelVertSurface::PerformCustomMenuEntry(int32 InEntryIndex)
{
	if (GIsEditor == true)
	{
		if (InEntryIndex == 0)
		{
			// Fill in the socket names array with the skeletal mesh 
			if (EditorSkelMesh != NULL)
			{
				// Retrieve all the bones
				if (EditorSkelMesh->GetRefSkeleton().GetRawBoneNum() > 0)
				{
					ValidAssociatedBones.Empty();
					for (int32 BoneIdx = 0; BoneIdx < EditorSkelMesh->GetRefSkeleton().GetRawBoneNum(); BoneIdx++)
					{
						int32 NewItemIdx = ValidAssociatedBones.AddZeroed();
						ValidAssociatedBones[NewItemIdx] = EditorSkelMesh->GetRefSkeleton().GetBoneName(BoneIdx);
					}
				}
				else
				{
					FMessageDialog::Open( EAppMsgType::Ok, NSLOCTEXT("UnrealEd", "Module_LocationBoneSocket_EditorMeshNoBones", "Editor mesh has no bones.") );
				}
			}
			else
			{
				FMessageDialog::Open( EAppMsgType::Ok, NSLOCTEXT("UnrealEd", "Module_LocationBoneSocket_NoEditorMesh", "No editor mesh is set.") );
			}
		}
	}
	return false;
}

bool UParticleModuleLocationSkelVertSurface::IsValidForLODLevel(UParticleLODLevel* LODLevel, FString& OutErrorString)
{
	if ( bInheritVertexColor && LODLevel && LODLevel->TypeDataModule && LODLevel->TypeDataModule->IsA<UParticleModuleTypeDataGpu>() )
	{
		OutErrorString = NSLOCTEXT("UnrealEd", "Module_LocationSkelVertSurface_InheritVertexColorOnGPUError", "Inherit Vertex Color is not supported on GPU emitters.").ToString();
		return false;
	}

	return true;
}

#endif

void UParticleModuleLocationSkelVertSurface::GetSkeletalMeshComponentSource(const FContext& Context, FModuleLocationVertSurfaceInstancePayload* InstancePayload)
{
	FParticleEmitterInstance* Owner = &Context.Owner;
	if (Owner == NULL)
	{
		return;
	}

	UParticleSystemComponent* PSysComp = Owner->Component.AsComponent();
	if (PSysComp == NULL)
	{
		return;
	}

	USkeletalMeshComponent* NewSkelMeshComp = nullptr;
	AActor* Actor = nullptr;
	PSysComp->GetActorParameter(SkelMeshActorParamName, Actor);
	bool bChangedActor = InstancePayload->CachedActor.Get() != Actor;
	if (!InstancePayload->SourceComponent.IsValid() || bChangedActor)
	{
		InstancePayload->SourceComponent = nullptr;
		InstancePayload->CachedActor = Actor;
		if(Actor == NULL)
		{
			return;
		}
		
		ASkeletalMeshActor* SkelMeshActor = Cast<ASkeletalMeshActor>(Actor);
		if (SkelMeshActor != NULL)
		{
			NewSkelMeshComp = SkelMeshActor->GetSkeletalMeshComponent();
		}
		else
		{
			USkeletalMeshComponent* SkeletalMeshComponent = Actor->FindComponentByClass<USkeletalMeshComponent>();
			if (SkeletalMeshComponent)
			{
				NewSkelMeshComp = SkeletalMeshComponent;
			}
			//@todo. Warn about this...
		}

		bool bMeshIsValid = false;
		int32 MinLOD = INDEX_NONE;
		if (NewSkelMeshComp && NewSkelMeshComp->GetScene() && NewSkelMeshComp->GetSkeletalMeshAsset())
		{
			FSkeletalMeshRenderData* SkelMeshResource = NewSkelMeshComp->GetSkeletalMeshRenderData();
			MinLOD = SkelMeshResource->GetFirstValidLODIdx(NewSkelMeshComp->GetSkeletalMeshAsset()->GetMinLodIdx());

			if (MinLOD != INDEX_NONE)
			{
				FSkinWeightVertexBuffer* SkinWeightBuffer = NewSkelMeshComp->GetSkinWeightBuffer(MinLOD);
				FSkeletalMeshLODRenderData& LODData = SkelMeshResource->LODRenderData[MinLOD];
				bool LODDataNumVerticesCorrect = LODData.GetNumVertices() > 0;
				bool LODDataPositonNumVerticesCorrect = LODData.StaticVertexBuffers.PositionVertexBuffer.GetNumVertices() > 0;
				bool bSkinWeightBuffer = SkinWeightBuffer != nullptr;
				bool SkinWeightBufferNumVerticesCorrect = bSkinWeightBuffer && (SkinWeightBuffer->GetNumVertices() > 0 && SkinWeightBuffer->GetNeedsCPUAccess());
				bool bIndexBufferValid = LODData.MultiSizeIndexContainer.IsIndexBufferValid();
				bool bIndexBufferFound = bIndexBufferValid && (LODData.MultiSizeIndexContainer.GetIndexBuffer() != nullptr);
				bool bIndexBufferNumCorrect = bIndexBufferFound && (LODData.MultiSizeIndexContainer.GetIndexBuffer()->Num() > 0);

				bMeshIsValid = LODDataNumVerticesCorrect &&
					LODDataPositonNumVerticesCorrect &&
					bSkinWeightBuffer &&
					SkinWeightBufferNumVerticesCorrect &&
					bIndexBufferValid &&
					bIndexBufferFound &&
					bIndexBufferNumCorrect;
			}
		}


		if (bMeshIsValid)
		{
			InstancePayload->MeshMinLOD = MinLOD;
			InstancePayload->SourceComponent = NewSkelMeshComp;
		}
		else
		{
			UE_LOGF(LogParticles, Warning, "----------------------------------------------------------------------");
			UE_LOGF(LogParticles, Warning, "Attempting to use Cascade SkelVertSurface module on mesh without valid data.");
			UE_LOGF(LogParticles, Warning, "Likely due to CPU side buffers being stripped with r.FreeSkeletalMeshBuffers=1.");
			UE_LOGF(LogParticles, Warning, "This emitter should probably be culled for this platform using detail mode.");
			UE_LOGF(LogParticles, Warning, "Mesh: %ls", NewSkelMeshComp ? *NewSkelMeshComp->GetSkeletalMeshAsset()->GetFullName() : TEXT("NULL"));
			UE_LOGF(LogParticles, Warning, "Comp: %ls", *Owner->Component.GetFullName());
			UE_LOGF(LogParticles, Warning, "System: %ls", *Owner->Component.GetTemplate()->GetFullName());
			UE_LOGF(LogParticles, Warning, "----------------------------------------------------------------------");

			Owner->Component.DeactivateNextTick();
			return;
		}

		UpdateBoneIndicesList(Owner);
	}
}


bool UParticleModuleLocationSkelVertSurface::GetParticleLocation(const FContext& Context, FModuleLocationVertSurfaceInstancePayload* InstPayload,
	USkeletalMeshComponent* InSkelMeshComponent, int32 InPrimaryVertexIndex, 
	FVector& OutPosition, FQuat& OutRotation, bool bSpawning /* = false*/)
{
	check(InSkelMeshComponent);
	checkSlow(InstPayload && InstPayload->MeshIsValid());
	FSkeletalMeshRenderData* SkelMeshResource = InSkelMeshComponent->GetSkeletalMeshRenderData();

	if (SkelMeshResource)
	{
		FSkeletalMeshLODRenderData& LODData = SkelMeshResource->LODRenderData[InstPayload->MeshMinLOD];
		FSkinWeightVertexBuffer& SkinWeightBuffer = *InSkelMeshComponent->GetSkinWeightBuffer(InstPayload->MeshMinLOD);
		if (SourceType == VERTSURFACESOURCE_Vert)
		{
			if ((uint32)InPrimaryVertexIndex >= LODData.GetNumVertices())
			{
				//possible if they change the mesh while the emitter has particles locked to verts/tris that are invalid on the new mesh.
				return false;
			}

			FVector VertPos(USkeletalMeshComponent::GetSkinnedVertexPosition(InSkelMeshComponent, InPrimaryVertexIndex, LODData, SkinWeightBuffer));
			OutPosition = InSkelMeshComponent->GetComponentTransform().TransformPosition(VertPos);
			OutRotation = FQuat::Identity;
		}
		else if (SourceType == VERTSURFACESOURCE_Surface)
		{
			FVector Verts[3];
			int32 VertIndex[3];
			int32 NumIndices = LODData.MultiSizeIndexContainer.GetIndexBuffer()->Num();
			if (InPrimaryVertexIndex + 2 >= NumIndices)
			{
				//possible if they change the mesh while the emitter has particles locked to verts/tris that are invalid on the new mesh.
				return false;
			}

			VertIndex[0] = LODData.MultiSizeIndexContainer.GetIndexBuffer()->Get( InPrimaryVertexIndex );
			VertIndex[1] = LODData.MultiSizeIndexContainer.GetIndexBuffer()->Get( InPrimaryVertexIndex+1 );
			VertIndex[2] = LODData.MultiSizeIndexContainer.GetIndexBuffer()->Get( InPrimaryVertexIndex+2 );
			Verts[0] = InSkelMeshComponent->GetComponentTransform().TransformPosition((FVector)USkeletalMeshComponent::GetSkinnedVertexPosition(InSkelMeshComponent, VertIndex[0], LODData, SkinWeightBuffer));
			Verts[1] = InSkelMeshComponent->GetComponentTransform().TransformPosition((FVector)USkeletalMeshComponent::GetSkinnedVertexPosition(InSkelMeshComponent, VertIndex[1], LODData, SkinWeightBuffer));
			Verts[2] = InSkelMeshComponent->GetComponentTransform().TransformPosition((FVector)USkeletalMeshComponent::GetSkinnedVertexPosition(InSkelMeshComponent, VertIndex[2], LODData, SkinWeightBuffer));

			FVector V0ToV2 = (Verts[2] - Verts[0]);
			V0ToV2.Normalize();
			FVector V0ToV1 = (Verts[1] - Verts[0]);
			V0ToV1.Normalize();
			FVector Normal = V0ToV2 ^ V0ToV1;
			Normal.Normalize();

			if(bEnforceNormalCheck && bSpawning)
			{
				float Dot = Normal | NormalToCompare;

				if(Dot < ((2.0f*NormalCheckTolerance)-1.0f))
				{
					return false;
				}

				OutPosition = (Verts[0] + Verts[1] + Verts[2]) / 3.0f;
			}
			else
			{
				OutPosition = (Verts[0] + Verts[1] + Verts[2]) / 3.0f;
			}

			if (bOrientMeshEmitters)
			{
				FVector Fwd = Normal ^ V0ToV1;
				Fwd.Normalize();
				FMatrix Orientation(FMatrix::Identity);
				Orientation.SetAxes(&V0ToV1, &Fwd, &Normal);
				OutRotation = FQuat(Orientation);
				OutRotation.Normalize();

			}
			else
			{
				OutRotation = FQuat::Identity;
			}
		}
	}

	if (Context.Owner.CurrentLODLevel->RequiredModule->bUseLocalSpace == true)
	{
		OutPosition = Context.GetTransform().InverseTransformPosition(OutPosition);
	}

	OutPosition += UniversalOffset;

	return true;
}


bool UParticleModuleLocationSkelVertSurface::VertInfluencedByActiveBone(FParticleEmitterInstance* Owner, FModuleLocationVertSurfaceInstancePayload* InstPayload, USkeletalMeshComponent* InSkelMeshComponent, int32 InVertexIndex, int32* OutBoneIndex)
{
	FSkeletalMeshRenderData* SkelMeshResource = InSkelMeshComponent->GetSkeletalMeshRenderData();
	if (SkelMeshResource)
	{
		check(InstPayload && InstPayload->MeshIsValid());
		FSkeletalMeshLODRenderData& LODData = SkelMeshResource->LODRenderData[InstPayload->MeshMinLOD];

		FModuleLocationVertSurfaceInstancePayload* InstancePayload = 
			(FModuleLocationVertSurfaceInstancePayload*)(Owner->GetModuleInstanceData(this));

		// Find the chunk and vertex within that chunk, and skinning type, for this vertex.
		int32 SectionIndex;
		int32 VertIndex;
		LODData.GetSectionFromVertexIndex(InVertexIndex, SectionIndex, VertIndex);

		check(SectionIndex < LODData.RenderSections.Num());

		FSkelMeshRenderSection& Section = LODData.RenderSections[SectionIndex];

		if (ValidMaterialIndices.Num() > 0)
		{
			// Does the material match one of the valid ones
			bool bFound = false;
			for (int32 ValidIdx = 0; ValidIdx < ValidMaterialIndices.Num(); ValidIdx++)
			{
				if (ValidMaterialIndices[ValidIdx] == Section.MaterialIndex)
				{
					bFound = true;
					break;
				}
			}

			if (!bFound)
			{
				// Material wasn't in the valid list...
				return false;
			}
		}

		return VertInfluencedByActiveBoneTyped(LODData, InstPayload->MeshMinLOD, Section, VertIndex, InSkelMeshComponent, InstancePayload, OutBoneIndex);
	}
	return false;
}

bool UParticleModuleLocationSkelVertSurface::VertInfluencedByActiveBoneTyped(
	FSkeletalMeshLODRenderData& LODData,
	int32 LODIndex,
	const FSkelMeshRenderSection& Section, 
	int32 VertIndex, 
	USkeletalMeshComponent* InSkelMeshComponent, 
	FModuleLocationVertSurfaceInstancePayload* InstancePayload, 
	int32* OutBoneIndex)
{
	const TArray<int32>& LeaderBoneMap = InSkelMeshComponent->GetLeaderBoneMap();
	// Get weights on this vertex
	FSkinWeightVertexBuffer* WeightBuffer = InSkelMeshComponent->GetSkinWeightBuffer(LODIndex);
	if (WeightBuffer)
	{
#if !PLATFORM_LITTLE_ENDIAN
		// uint8[] elements in LOD.VertexBufferGPUSkin have been swapped for VET_UBYTE4 vertex stream use
		for (int32 InfluenceIndex = MAX_INFLUENCES - 1; InfluenceIndex >= MAX_INFLUENCES - Section.MaxBoneInfluences; InfluenceIndex--)
#else
		for (int32 InfluenceIndex = 0; InfluenceIndex < Section.MaxBoneInfluences; InfluenceIndex++)
#endif
		{
			int32 BoneIndex = Section.BoneMap[WeightBuffer->GetBoneIndex(Section.GetVertexBufferIndex() + VertIndex, InfluenceIndex)];
			if (InSkelMeshComponent->LeaderPoseComponent.IsValid())
			{
				check(LeaderBoneMap.Num() == InSkelMeshComponent->GetSkeletalMeshAsset()->GetRefSkeleton().GetNum());
				BoneIndex = LeaderBoneMap[BoneIndex];
			}

			if (!InstancePayload->NumValidAssociatedBoneIndices || InstancePayload->ValidAssociatedBoneIndices.Contains(BoneIndex))
			{
				if (OutBoneIndex)
				{
					*OutBoneIndex = BoneIndex;
				}

				return true;
			}
		}
	}

	return false;
}
