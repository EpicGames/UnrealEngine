// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	ParticleModules_Material.cpp: 
	Material-related particle module implementations.
=============================================================================*/

#include "Particles/Material/ParticleModuleMaterialBase.h"
#include "Particles/Material/ParticleModuleMeshMaterial.h"
#include "Particles/ParticleModule.h"
#include "Particles/TypeData/ParticleModuleTypeDataMesh.h"
#include "Particles/ParticleModuleRequired.h"
#include "Particles/ParticleLODLevel.h"
#include "Templates/Casts.h"

UParticleModuleMaterialBase::UParticleModuleMaterialBase(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

/*-----------------------------------------------------------------------------
	Abstract base modules used for categorization.
-----------------------------------------------------------------------------*/

/*-----------------------------------------------------------------------------
	UParticleModuleMeshMaterial
-----------------------------------------------------------------------------*/
UParticleModuleMeshMaterial::UParticleModuleMeshMaterial(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bSpawnModule = true;
	bUpdateModule = true;
}

	//## BEGIN PROPS ParticleModuleMeshMaterial
//	TArray<class UMaterialInstance*> MeshMaterials;
	//## END PROPS ParticleModuleMeshMaterial

void UParticleModuleMeshMaterial::Spawn(const FSpawnContext& Context)
{

}

uint32 UParticleModuleMeshMaterial::RequiredBytesPerInstance()
{
	return 0;
}

#if WITH_EDITOR

bool UParticleModuleMeshMaterial::IsValidForLODLevel(UParticleLODLevel* LODLevel, FString& OutErrorString)
{
	if (LODLevel->RequiredModule->NamedMaterialOverrides.Num() > 0)
	{
		OutErrorString = NSLOCTEXT("UnrealEd", "MeshMaterialsWithNamedMaterialsError", "Cannot use Mesh Materials Module when using Named Material Overrides in the required module.").ToString();
		return false;
	}

	if (UParticleModuleTypeDataMesh* MeshTypeData = Cast<UParticleModuleTypeDataMesh>(LODLevel->TypeDataModule))
	{
		if (MeshTypeData->bOverrideMaterial)
		{
			OutErrorString = NSLOCTEXT("UnrealEd", "MeshMaterialsWithOverrideMaterialError", "Cannot use Mesh Materials Module when using OverrideMaterial in the mesh type data module.").ToString();
			return false;
		}
	}

	return true;
}

#endif
