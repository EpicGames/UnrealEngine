// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	ParticleModules_Orbit.cpp: Orbit particle modules implementation.
=============================================================================*/

#include "ParticleEmitterInstances.h"
#include "Particles/ParticleSystemComponent.h"
#include "Distributions/DistributionVectorUniform.h"
#include "Particles/Orbit/ParticleModuleOrbitBase.h"
#include "Particles/Orbit/ParticleModuleOrbit.h"
#include "Particles/ParticleModule.h"
#include "Particles/TypeData/ParticleModuleTypeDataGpu.h"
#include "Particles/ParticleLODLevel.h"
#include "Particles/TypeData/ParticleModuleTypeDataBase.h"

UParticleModuleOrbitBase::UParticleModuleOrbitBase(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}


/*-----------------------------------------------------------------------------
	Abstract base modules used for categorization.
-----------------------------------------------------------------------------*/

/*-----------------------------------------------------------------------------
	UParticleModuleOrbit implementation.
-----------------------------------------------------------------------------*/
UParticleModuleOrbit::UParticleModuleOrbit(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bSpawnModule = true;
	bUpdateModule = true;
	ChainMode = EOChainMode_Add;
}

void UParticleModuleOrbit::InitializeDefaults()
{
	if (!OffsetAmount.IsCreated())
	{
		UDistributionVectorUniform* DistributionOffsetAmount = NewObject<UDistributionVectorUniform>(this, TEXT("DistributionOffsetAmount"));
		DistributionOffsetAmount->Min = FVector(0.0f, 0.0f, 0.0f);
		DistributionOffsetAmount->Max = FVector(0.0f, 50.0f, 0.0f);
		OffsetAmount.Distribution = DistributionOffsetAmount;
	}

	if (!RotationAmount.IsCreated())
	{
		UDistributionVectorUniform* DistributionRotationAmount = NewObject<UDistributionVectorUniform>(this, TEXT("DistributionRotationAmount"));
		DistributionRotationAmount->Min = FVector(0.0f, 0.0f, 0.0f);
		DistributionRotationAmount->Max = FVector(1.0f, 1.0f, 1.0f);
		RotationAmount.Distribution = DistributionRotationAmount;
	}

	if (!RotationRateAmount.IsCreated())
	{
		UDistributionVectorUniform* DistributionRotationRateAmount = NewObject<UDistributionVectorUniform>(this, TEXT("DistributionRotationRateAmount"));
		DistributionRotationRateAmount->Min = FVector(0.0f, 0.0f, 0.0f);
		DistributionRotationRateAmount->Max = FVector(1.0f, 1.0f, 1.0f);
		RotationRateAmount.Distribution = DistributionRotationRateAmount;
	}
}

void UParticleModuleOrbit::PostInitProperties()
{
	Super::PostInitProperties();
	if (!HasAnyFlags(RF_ClassDefaultObject | RF_NeedLoad))
	{
		InitializeDefaults();
	}
}

void UParticleModuleOrbit::CompileModule(FParticleEmitterBuildInfo& EmitterInfo)
{
	switch (ChainMode)
	{
	case EOChainMode_Add:
		{
			EmitterInfo.OrbitOffset.AddDistribution(OffsetAmount.Distribution);
			EmitterInfo.OrbitInitialRotation.AddDistribution(RotationAmount.Distribution);
			EmitterInfo.OrbitRotationRate.AddDistribution(RotationRateAmount.Distribution);
		}
		break;

	case EOChainMode_Scale:
		{
			EmitterInfo.OrbitOffset.ScaleByVectorDistribution(OffsetAmount.Distribution);
			EmitterInfo.OrbitInitialRotation.ScaleByVectorDistribution(RotationAmount.Distribution);
			EmitterInfo.OrbitRotationRate.ScaleByVectorDistribution(RotationRateAmount.Distribution);
		}
		break;

	case EOChainMode_Link:
		{
			EmitterInfo.OrbitOffset.Initialize(OffsetAmount.Distribution);
			EmitterInfo.OrbitInitialRotation.Initialize(RotationAmount.Distribution);
			EmitterInfo.OrbitRotationRate.Initialize(RotationRateAmount.Distribution);
		}
		break;
	}
}

#if WITH_EDITOR
void UParticleModuleOrbit::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	InitializeDefaults();
	Super::PostEditChangeProperty(PropertyChangedEvent);
}
#endif // WITH_EDITOR

void UParticleModuleOrbit::Spawn(const FSpawnContext& Context)
{
	FParticleEmitterInstance* Owner = &Context.Owner;

	SPAWN_INIT;
	{
		PARTICLE_ELEMENT(FOrbitChainModuleInstancePayload, OrbitPayload);

		if (OffsetOptions.bProcessDuringSpawn == true)
		{
			// Process the offset
			FVector LocalOffset;
			if (OffsetOptions.bUseEmitterTime == false)
			{
				LocalOffset = OffsetAmount.GetValue(Particle.RelativeTime, Context.GetDistributionData());
			}
			else
			{
				LocalOffset = OffsetAmount.GetValue(Owner->EmitterTime, Context.GetDistributionData());
			}
			OrbitPayload.BaseOffset += (FVector3f)LocalOffset;
			OrbitPayload.PreviousOffset = OrbitPayload.Offset;
			OrbitPayload.Offset += (FVector3f)LocalOffset;
		}

		if (RotationOptions.bProcessDuringSpawn == true)
		{
			// Process the rotation
			FVector LocalRotation;
			if (RotationOptions.bUseEmitterTime == false)
			{
				LocalRotation = RotationAmount.GetValue(Particle.RelativeTime, Context.GetDistributionData());
			}
			else
			{
				LocalRotation = RotationAmount.GetValue(Owner->EmitterTime, Context.GetDistributionData());
			}
			OrbitPayload.Rotation += (FVector3f)LocalRotation;
		}

		if (RotationRateOptions.bProcessDuringSpawn == true)
		{
			// Process the rotation rate
			FVector LocalRotationRate;
			if (RotationRateOptions.bUseEmitterTime == false)
			{
				LocalRotationRate = RotationRateAmount.GetValue(Particle.RelativeTime, Context.GetDistributionData());
			}
			else
			{
				LocalRotationRate = RotationRateAmount.GetValue(Owner->EmitterTime, Context.GetDistributionData());
			}
			OrbitPayload.BaseRotationRate += (FVector3f)LocalRotationRate;
			OrbitPayload.RotationRate += (FVector3f)LocalRotationRate;
		}
	}
}

void UParticleModuleOrbit::Update(const FUpdateContext& Context)
{
	FParticleEmitterInstance* Owner = &Context.Owner;
	BEGIN_UPDATE_LOOP;
	{
		PARTICLE_ELEMENT(FOrbitChainModuleInstancePayload, OrbitPayload);

		if (OffsetOptions.bProcessDuringUpdate == true)
		{
			// Process the offset
			FVector LocalOffset;
			if (OffsetOptions.bUseEmitterTime == false)
			{
				LocalOffset = OffsetAmount.GetValue(Particle.RelativeTime, Context.GetDistributionData());
			}
			else
			{
				LocalOffset = OffsetAmount.GetValue(Owner->EmitterTime, Context.GetDistributionData());
			}

			//@todo. Do we need to update the base offset here???
//			OrbitPayload.BaseOffset += LocalOffset;
			OrbitPayload.PreviousOffset = OrbitPayload.Offset;
			OrbitPayload.Offset += (FVector3f)LocalOffset;
		}

		if (RotationOptions.bProcessDuringUpdate == true)
		{
			// Process the rotation
			FVector LocalRotation;
			if (RotationOptions.bUseEmitterTime == false)
			{
				LocalRotation = RotationAmount.GetValue(Particle.RelativeTime, Context.GetDistributionData());
			}
			else
			{
				LocalRotation = RotationAmount.GetValue(Owner->EmitterTime, Context.GetDistributionData());
			}
			OrbitPayload.Rotation += (FVector3f)LocalRotation;
		}


		if (RotationRateOptions.bProcessDuringUpdate == true)
		{
			// Process the rotation rate
			FVector LocalRotationRate;
			if (RotationRateOptions.bUseEmitterTime == false)
			{
				LocalRotationRate = RotationRateAmount.GetValue(Particle.RelativeTime, Context.GetDistributionData());
			}
			else
			{
				LocalRotationRate = RotationRateAmount.GetValue(Owner->EmitterTime, Context.GetDistributionData());
			}
			//@todo. Do we need to update the base rotationrate here???
//			OrbitPayload.BaseRotationRate += LocalRotationRate;
			OrbitPayload.RotationRate += (FVector3f)LocalRotationRate;
		}
	}
	END_UPDATE_LOOP;
}

uint32 UParticleModuleOrbit::RequiredBytes(UParticleModuleTypeDataBase* TypeData)
{
	return sizeof(FOrbitChainModuleInstancePayload);
}

uint32 UParticleModuleOrbit::RequiredBytesPerInstance()
{
	return 0;
}

#if WITH_EDITOR
bool UParticleModuleOrbit::IsValidForLODLevel(UParticleLODLevel* LODLevel, FString& OutErrorString)
{
	if (LODLevel->TypeDataModule && LODLevel->TypeDataModule->IsA(UParticleModuleTypeDataGpu::StaticClass()))
	{
		if(!IsDistributionAllowedOnGPU(OffsetAmount.Distribution))
		{
			OutErrorString = GetDistributionNotAllowedOnGPUText(StaticClass()->GetName(), "OffsetAmount" ).ToString();
			return false;
		}

		if(!IsDistributionAllowedOnGPU(RotationAmount.Distribution))
		{
			OutErrorString = GetDistributionNotAllowedOnGPUText(StaticClass()->GetName(), "RotationAmount" ).ToString();
			return false;
		}

		if(!IsDistributionAllowedOnGPU(RotationRateAmount.Distribution))
		{
			OutErrorString = GetDistributionNotAllowedOnGPUText(StaticClass()->GetName(), "RotationRateAmount" ).ToString();
			return false;
		}
	}

	return true;
}
#endif
