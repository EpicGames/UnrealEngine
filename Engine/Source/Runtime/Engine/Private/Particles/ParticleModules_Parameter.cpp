// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	ParticleModules_Parameter.cpp: 
	Parameter-related particle module implementations.
=============================================================================*/

#include "ParticleEmitterInstances.h"
#include "Particles/ParticleModule.h"
#include "Distributions/DistributionFloatConstant.h"
#include "MaterialCachedData.h"
#include "Particles/Parameter/ParticleModuleParameterBase.h"
#include "Particles/Parameter/ParticleModuleParameterDynamic.h"
#include "Particles/Parameter/ParticleModuleParameterDynamic_Seeded.h"
#include "Particles/ParticleEmitter.h"
#include "Particles/TypeData/ParticleModuleTypeDataMesh.h"
#include "Particles/ParticleLODLevel.h"
#include "Particles/ParticleModuleRequired.h"
#include "Particles/Material/ParticleModuleMeshMaterial.h"
#include "Engine/InterpCurveEdSetup.h"
#include "Particles/TypeData/ParticleModuleTypeDataBase.h"

UParticleModuleParameterBase::UParticleModuleParameterBase(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

/*-----------------------------------------------------------------------------
	Abstract base modules used for categorization.
-----------------------------------------------------------------------------*/

/*-----------------------------------------------------------------------------
	UParticleModuleParameterDynamic implementation.
-----------------------------------------------------------------------------*/
UParticleModuleParameterDynamic::UParticleModuleParameterDynamic(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bSpawnModule = true;
	bUpdateModule = true;
}

void UParticleModuleParameterDynamic::InitializeDefaults()
{
	for(int32 ParamIdx = 0; ParamIdx < DynamicParams.Num(); ++ParamIdx)
	{
		if(!DynamicParams[ParamIdx].ParamValue.IsCreated())
		{
			DynamicParams[ParamIdx].ParamValue.Distribution = NewObject<UDistributionFloatConstant>(this, TEXT("DistributionParam1"));
		}
	}
}

void UParticleModuleParameterDynamic::PostInitProperties()
{
	Super::PostInitProperties();
	if (!HasAnyFlags(RF_ClassDefaultObject | RF_NeedLoad))
	{
		// Structure to hold one-time initialization
		struct FConstructorStatics
		{
			FName NAME_None;
			FConstructorStatics()
				: NAME_None(TEXT("None"))
			{
			}
		};
		static FConstructorStatics ConstructorStatics;

		UDistributionFloatConstant* NewParam1 = NewObject<UDistributionFloatConstant>(this, TEXT("DistributionParam1")); NewParam1->Constant = 1.0f;
		UDistributionFloatConstant* NewParam2 = NewObject<UDistributionFloatConstant>(this, TEXT("DistributionParam2")); NewParam2->Constant = 1.0f;
		UDistributionFloatConstant* NewParam3 = NewObject<UDistributionFloatConstant>(this, TEXT("DistributionParam3")); NewParam3->Constant = 1.0f;
		UDistributionFloatConstant* NewParam4 = NewObject<UDistributionFloatConstant>(this, TEXT("DistributionParam4")); NewParam4->Constant = 1.0f;

		DynamicParams.Add(FEmitterDynamicParameter(ConstructorStatics.NAME_None, false, EDPV_UserSet, NewParam1));
		DynamicParams.Add(FEmitterDynamicParameter(ConstructorStatics.NAME_None, false, EDPV_UserSet, NewParam2));
		DynamicParams.Add(FEmitterDynamicParameter(ConstructorStatics.NAME_None, false, EDPV_UserSet, NewParam3));
		DynamicParams.Add(FEmitterDynamicParameter(ConstructorStatics.NAME_None, false, EDPV_UserSet, NewParam4));
	}
}

/** Flags for optimizing update */
enum EDynamicParameterUpdateFlags
{
	/** No Update required */
	EDPU_UPDATE_NONE	= 0x00,
	/** Param1 requires an update */
	EDPU_UPDATE_0		= 0x01,
	/** Param2 requires an update */
	EDPU_UPDATE_1		= 0x02,
	/** Param3 requires an update */
	EDPU_UPDATE_2		= 0x04,
	/** Param4 requires an update */
	EDPU_UPDATE_3		= 0x08,
	/** Param1 and Param2 require an update */
	EDPU_UPDATE_01		= EDPU_UPDATE_0	| EDPU_UPDATE_1,
	/** Param1, Param2, and Param3 require an update */
	EDPU_UPDATE_012		= EDPU_UPDATE_0	| EDPU_UPDATE_1	| EDPU_UPDATE_2,
	/** ALL require an update */
	EDPU_UPDATE_ALL		= EDPU_UPDATE_0 | EDPU_UPDATE_1 | EDPU_UPDATE_2 | EDPU_UPDATE_3
};

FORCEINLINE int32 ParticleDynamicParameter_GetIndexFromFlag(int32 InFlags)
{
	switch (InFlags)
	{
	case EDPU_UPDATE_0:
		return 0;
	case EDPU_UPDATE_1:
		return 1;
	case EDPU_UPDATE_2:
		return 2;
	case EDPU_UPDATE_3:
		return 3;
	}
	return INDEX_NONE;
}

void UParticleModuleParameterDynamic::PostLoad()
{
	Super::PostLoad();
	UpdateUsageFlags();
}

bool UParticleModuleParameterDynamic::CanTickInAnyThread()
{
	for (FEmitterDynamicParameter& Parm : DynamicParams)
	{
		if (!Parm.ParamValue.OkForParallel())
		{
			return false;
		}
	}
	return true;
}


void UParticleModuleParameterDynamic::Spawn(const FSpawnContext& Context)
{
	SpawnEx(Context, &GetRandomStream(Context));
}

void UParticleModuleParameterDynamic::SpawnEx(const FSpawnContext& Context, struct FRandomStream* InRandomStream)
{
	FParticleEmitterInstance* Owner = &Context.Owner;

	SPAWN_INIT;
	{
		PARTICLE_ELEMENT(FEmitterDynamicParameterPayload, DynamicPayload);
		if (DynamicParams[0].ValueMethod != EDPV_AutoSet)
		{
			DynamicPayload.DynamicParameterValue[0] = GetParameterValue(Context, DynamicParams[0], Particle, Owner, InRandomStream);
		}
		if (DynamicParams[1].ValueMethod != EDPV_AutoSet)
		{
			DynamicPayload.DynamicParameterValue[1] = GetParameterValue(Context, DynamicParams[1], Particle, Owner, InRandomStream);
		}
		if (DynamicParams[2].ValueMethod != EDPV_AutoSet)
		{
			DynamicPayload.DynamicParameterValue[2] = GetParameterValue(Context, DynamicParams[2], Particle, Owner, InRandomStream);
		}
		if (DynamicParams[3].ValueMethod != EDPV_AutoSet)
		{
			DynamicPayload.DynamicParameterValue[3] = GetParameterValue(Context, DynamicParams[3], Particle, Owner, InRandomStream);
		}
	}
}

void UParticleModuleParameterDynamic::Update(const FUpdateContext& Context)
{
	if (UpdateFlags == EDPU_UPDATE_NONE)
	{
		// Nothing to do here - they are all spawntime only
		return;
	}

	FParticleEmitterInstance* Owner = &Context.Owner;
	if ((Owner == NULL) || (Owner->ActiveParticles <= 0) || 
		(Owner->ParticleData == NULL) || (Owner->ParticleIndices == NULL))
	{
		return;
	}

	FRandomStream& RandomStream = GetRandomStream(Context);

	// 
	int32 ParameterIndex = ParticleDynamicParameter_GetIndexFromFlag(UpdateFlags);

	FPlatformMisc::Prefetch(Owner->ParticleData, (Owner->ParticleIndices[0] * Owner->ParticleStride));
	FPlatformMisc::Prefetch(Owner->ParticleData, (Owner->ParticleIndices[0] * Owner->ParticleStride) + PLATFORM_CACHE_LINE_SIZE);

	switch (UpdateFlags)
	{
	case EDPU_UPDATE_0:
	case EDPU_UPDATE_1:
	case EDPU_UPDATE_2:
	case EDPU_UPDATE_3:
		{
			// Only one parameter is updating...
			check(ParameterIndex != INDEX_NONE);
			FEmitterDynamicParameter& DynParam = DynamicParams[ParameterIndex];
			if (bUsesVelocity == false)
			{
				BEGIN_UPDATE_LOOP;
				{
					FEmitterDynamicParameterPayload& DynamicPayload = *((FEmitterDynamicParameterPayload*)(ParticleBase + CurrentOffset));
					FPlatformMisc::Prefetch(ParticleData, (ParticleIndices[i+1] * ParticleStride));
					FPlatformMisc::Prefetch(ParticleData, (ParticleIndices[i+1] * ParticleStride) + PLATFORM_CACHE_LINE_SIZE);
					DynamicPayload.DynamicParameterValue[ParameterIndex] = GetParameterValue_UserSet(Context, DynParam, Particle, Owner, &RandomStream);
				}
				END_UPDATE_LOOP;
			}
			else
			{
				BEGIN_UPDATE_LOOP;
				{
					FEmitterDynamicParameterPayload& DynamicPayload = *((FEmitterDynamicParameterPayload*)(ParticleBase + CurrentOffset));
					FPlatformMisc::Prefetch(ParticleData, (ParticleIndices[i+1] * ParticleStride));
					FPlatformMisc::Prefetch(ParticleData, (ParticleIndices[i+1] * ParticleStride) + PLATFORM_CACHE_LINE_SIZE);
					DynamicPayload.DynamicParameterValue[ParameterIndex] = GetParameterValue(Context, DynParam, Particle, Owner, &RandomStream);
				}
				END_UPDATE_LOOP;
			}
		}
		break;
	case EDPU_UPDATE_01:
		{
			// Just 0 and 1 need to be updated...
			FEmitterDynamicParameter& DynParam0 = DynamicParams[0];
			FEmitterDynamicParameter& DynParam1 = DynamicParams[1];
			if (bUsesVelocity == false)
			{
				BEGIN_UPDATE_LOOP;
				{
					FEmitterDynamicParameterPayload& DynamicPayload = *((FEmitterDynamicParameterPayload*)(ParticleBase + CurrentOffset));
					FPlatformMisc::Prefetch(ParticleData, (ParticleIndices[i+1] * ParticleStride));
					FPlatformMisc::Prefetch(ParticleData, (ParticleIndices[i+1] * ParticleStride) + PLATFORM_CACHE_LINE_SIZE);
					DynamicPayload.DynamicParameterValue[0] = GetParameterValue_UserSet(Context, DynParam0, Particle, Owner, &RandomStream);
					DynamicPayload.DynamicParameterValue[1] = GetParameterValue_UserSet(Context, DynParam1, Particle, Owner, &RandomStream);
				}
				END_UPDATE_LOOP;
			}
			else
			{
				BEGIN_UPDATE_LOOP;
				{
					FEmitterDynamicParameterPayload& DynamicPayload = *((FEmitterDynamicParameterPayload*)(ParticleBase + CurrentOffset));
					FPlatformMisc::Prefetch(ParticleData, (ParticleIndices[i+1] * ParticleStride));
					FPlatformMisc::Prefetch(ParticleData, (ParticleIndices[i+1] * ParticleStride) + PLATFORM_CACHE_LINE_SIZE);
					DynamicPayload.DynamicParameterValue[0] = GetParameterValue(Context, DynParam0, Particle, Owner, &RandomStream);
					DynamicPayload.DynamicParameterValue[1] = GetParameterValue(Context, DynParam1, Particle, Owner, &RandomStream);
				}
				END_UPDATE_LOOP;
			}
		}
		break;
	case EDPU_UPDATE_012:
		{
			// Just 0, 1 and 2 need to be updated...
			FEmitterDynamicParameter& DynParam0 = DynamicParams[0];
			FEmitterDynamicParameter& DynParam1 = DynamicParams[1];
			FEmitterDynamicParameter& DynParam2 = DynamicParams[2];
			if (bUsesVelocity == false)
			{
				BEGIN_UPDATE_LOOP;
				{
					FEmitterDynamicParameterPayload& DynamicPayload = *((FEmitterDynamicParameterPayload*)(ParticleBase + CurrentOffset));
					FPlatformMisc::Prefetch(ParticleData, (ParticleIndices[i+1] * ParticleStride));
					FPlatformMisc::Prefetch(ParticleData, (ParticleIndices[i+1] * ParticleStride) + PLATFORM_CACHE_LINE_SIZE);
					DynamicPayload.DynamicParameterValue[0] = GetParameterValue_UserSet(Context, DynParam0, Particle, Owner, &RandomStream);
					DynamicPayload.DynamicParameterValue[1] = GetParameterValue_UserSet(Context, DynParam1, Particle, Owner, &RandomStream);
					DynamicPayload.DynamicParameterValue[2] = GetParameterValue_UserSet(Context, DynParam2, Particle, Owner, &RandomStream);
				}
				END_UPDATE_LOOP;
			}
			else
			{
				BEGIN_UPDATE_LOOP;
				{
					FEmitterDynamicParameterPayload& DynamicPayload = *((FEmitterDynamicParameterPayload*)(ParticleBase + CurrentOffset));
					FPlatformMisc::Prefetch(ParticleData, (ParticleIndices[i+1] * ParticleStride));
					FPlatformMisc::Prefetch(ParticleData, (ParticleIndices[i+1] * ParticleStride) + PLATFORM_CACHE_LINE_SIZE);
					DynamicPayload.DynamicParameterValue[0] = GetParameterValue(Context, DynParam0, Particle, Owner, &RandomStream);
					DynamicPayload.DynamicParameterValue[1] = GetParameterValue(Context, DynParam1, Particle, Owner, &RandomStream);
					DynamicPayload.DynamicParameterValue[2] = GetParameterValue(Context, DynParam2, Particle, Owner, &RandomStream);
				}
				END_UPDATE_LOOP;
			}
		}
		break;
	case EDPU_UPDATE_ALL:
		{
			FEmitterDynamicParameter& DynParam0 = DynamicParams[0];
			FEmitterDynamicParameter& DynParam1 = DynamicParams[1];
			FEmitterDynamicParameter& DynParam2 = DynamicParams[2];
			FEmitterDynamicParameter& DynParam3 = DynamicParams[3];
			if (bUsesVelocity == false)
			{
				BEGIN_UPDATE_LOOP;
				{
					FEmitterDynamicParameterPayload& DynamicPayload = *((FEmitterDynamicParameterPayload*)(ParticleBase + CurrentOffset));
					FPlatformMisc::Prefetch(ParticleData, (ParticleIndices[i+1] * ParticleStride));
					FPlatformMisc::Prefetch(ParticleData, (ParticleIndices[i+1] * ParticleStride) + PLATFORM_CACHE_LINE_SIZE);
					DynamicPayload.DynamicParameterValue[0] = GetParameterValue_UserSet(Context, DynParam0, Particle, Owner, &RandomStream);
					DynamicPayload.DynamicParameterValue[1] = GetParameterValue_UserSet(Context, DynParam1, Particle, Owner, &RandomStream);
					DynamicPayload.DynamicParameterValue[2] = GetParameterValue_UserSet(Context, DynParam2, Particle, Owner, &RandomStream);
					DynamicPayload.DynamicParameterValue[3] = GetParameterValue_UserSet(Context, DynParam3, Particle, Owner, &RandomStream);
				}
				END_UPDATE_LOOP;
			}
			else
			{
				BEGIN_UPDATE_LOOP;
				{
					FEmitterDynamicParameterPayload& DynamicPayload = *((FEmitterDynamicParameterPayload*)(ParticleBase + CurrentOffset));
					FPlatformMisc::Prefetch(ParticleData, (ParticleIndices[i+1] * ParticleStride));
					FPlatformMisc::Prefetch(ParticleData, (ParticleIndices[i+1] * ParticleStride) + PLATFORM_CACHE_LINE_SIZE);
					DynamicPayload.DynamicParameterValue[0] = GetParameterValue(Context, DynParam0, Particle, Owner, &RandomStream);
					DynamicPayload.DynamicParameterValue[1] = GetParameterValue(Context, DynParam1, Particle, Owner, &RandomStream);
					DynamicPayload.DynamicParameterValue[2] = GetParameterValue(Context, DynParam2, Particle, Owner, &RandomStream);
					DynamicPayload.DynamicParameterValue[3] = GetParameterValue(Context, DynParam3, Particle, Owner, &RandomStream);
				}
				END_UPDATE_LOOP;
			}
		}
		break;
	default:
		{
			FEmitterDynamicParameter& DynParam0 = DynamicParams[0];
			FEmitterDynamicParameter& DynParam1 = DynamicParams[1];
			FEmitterDynamicParameter& DynParam2 = DynamicParams[2];
			FEmitterDynamicParameter& DynParam3 = DynamicParams[3];
			if (bUsesVelocity == false)
			{
				BEGIN_UPDATE_LOOP;
				{
					FEmitterDynamicParameterPayload& DynamicPayload = *((FEmitterDynamicParameterPayload*)(ParticleBase + CurrentOffset));
					FPlatformMisc::Prefetch(ParticleData, (ParticleIndices[i+1] * ParticleStride));
					FPlatformMisc::Prefetch(ParticleData, (ParticleIndices[i+1] * ParticleStride) + PLATFORM_CACHE_LINE_SIZE);
					DynamicPayload.DynamicParameterValue[0] = (UpdateFlags & EDPU_UPDATE_0) ? GetParameterValue_UserSet(Context, DynParam0, Particle, Owner, &RandomStream) : DynamicPayload.DynamicParameterValue[0];
					DynamicPayload.DynamicParameterValue[1] = (UpdateFlags & EDPU_UPDATE_1) ? GetParameterValue_UserSet(Context, DynParam1, Particle, Owner, &RandomStream) : DynamicPayload.DynamicParameterValue[1];
					DynamicPayload.DynamicParameterValue[2] = (UpdateFlags & EDPU_UPDATE_2) ? GetParameterValue_UserSet(Context, DynParam2, Particle, Owner, &RandomStream) : DynamicPayload.DynamicParameterValue[2];
					DynamicPayload.DynamicParameterValue[3] = (UpdateFlags & EDPU_UPDATE_3) ? GetParameterValue_UserSet(Context, DynParam3, Particle, Owner, &RandomStream) : DynamicPayload.DynamicParameterValue[3];
				}
				END_UPDATE_LOOP;
			}
			else
			{
				BEGIN_UPDATE_LOOP;
				{
					FEmitterDynamicParameterPayload& DynamicPayload = *((FEmitterDynamicParameterPayload*)(ParticleBase + CurrentOffset));
					FPlatformMisc::Prefetch(ParticleData, (ParticleIndices[i+1] * ParticleStride));
					FPlatformMisc::Prefetch(ParticleData, (ParticleIndices[i+1] * ParticleStride) + PLATFORM_CACHE_LINE_SIZE);
					DynamicPayload.DynamicParameterValue[0] = (UpdateFlags & EDPU_UPDATE_0) ? GetParameterValue(Context, DynParam0, Particle, Owner, &RandomStream) : DynamicPayload.DynamicParameterValue[0];
					DynamicPayload.DynamicParameterValue[1] = (UpdateFlags & EDPU_UPDATE_1) ? GetParameterValue(Context, DynParam1, Particle, Owner, &RandomStream) : DynamicPayload.DynamicParameterValue[1];
					DynamicPayload.DynamicParameterValue[2] = (UpdateFlags & EDPU_UPDATE_2) ? GetParameterValue(Context, DynParam2, Particle, Owner, &RandomStream) : DynamicPayload.DynamicParameterValue[2];
					DynamicPayload.DynamicParameterValue[3] = (UpdateFlags & EDPU_UPDATE_3) ? GetParameterValue(Context, DynParam3, Particle, Owner, &RandomStream) : DynamicPayload.DynamicParameterValue[3];
				}
				END_UPDATE_LOOP;
			}
		}
		break;
	}
}

uint32 UParticleModuleParameterDynamic::RequiredBytes(UParticleModuleTypeDataBase* TypeData)
{
	return sizeof(FEmitterDynamicParameterPayload);
}

void UParticleModuleParameterDynamic::SetToSensibleDefaults(UParticleEmitter* Owner)
{
}

#if WITH_EDITOR
void UParticleModuleParameterDynamic::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	InitializeDefaults();
	UpdateUsageFlags();
	Super::PostEditChangeProperty(PropertyChangedEvent);
}
#endif // WITH_EDITOR

void UParticleModuleParameterDynamic::GetCurveObjects(TArray<FParticleCurvePair>& OutCurves)
{
	FParticleCurvePair* NewCurve;
	
	for (int32 ParamIndex = 0; ParamIndex < 4; ParamIndex++)
	{
		NewCurve = new(OutCurves) FParticleCurvePair;
		check(NewCurve);
		NewCurve->CurveObject = DynamicParams[ParamIndex].ParamValue.Distribution;
		NewCurve->CurveName = FString::Printf(TEXT("%s (DP%d)"), 
			*(DynamicParams[ParamIndex].ParamName.ToString()), ParamIndex);
	}
}

void UParticleModuleParameterDynamic::GetParticleSysParamsUtilized(TArray<FString>& ParticleSysParamList)
{
}

void UParticleModuleParameterDynamic::GetParticleParametersUtilized(TArray<FString>& ParticleParameterList)
{
}

void UParticleModuleParameterDynamic::UpdateParameterNames(UMaterialInterface* InMaterialInterface)
{
	const TArray<FName>& ParamNames = InMaterialInterface->GetCachedExpressionData().DynamicParameterNames;
	check(DynamicParams.Num() == 4);
	for (int32 ParamIndex = 0; ParamIndex < 4; ParamIndex++)
	{
		if (ParamNames.IsValidIndex(ParamIndex))
		{
			DynamicParams[ParamIndex].ParamName = ParamNames[ParamIndex];
		}
		else
		{
			DynamicParams[ParamIndex].ParamName = NAME_None;
		}
	}
}

void UParticleModuleParameterDynamic::RefreshModule(UInterpCurveEdSetup* EdSetup, UParticleEmitter* InEmitter, int32 InLODLevel)
{
#if WITH_EDITOR
	// Find the material for this emitter...
	UParticleLODLevel* LODLevel = InEmitter->LODLevels[(InLODLevel < InEmitter->LODLevels.Num()) ? InLODLevel : 0];
	if (LODLevel)
	{
		bool bIsMeshEmitter = false;
		if (LODLevel->TypeDataModule)
		{
			if (LODLevel->TypeDataModule->IsA(UParticleModuleTypeDataMesh::StaticClass()))
			{
				bIsMeshEmitter = true;
			}
		}

		UMaterialInterface* MaterialInterface = LODLevel->RequiredModule ? LODLevel->RequiredModule->Material : nullptr;
		if( bIsMeshEmitter )
		{
			UParticleModuleMeshMaterial* MeshMaterialModule = nullptr;
			LODLevel->Modules.FindItemByClass( &MeshMaterialModule );
			if( MeshMaterialModule && MeshMaterialModule->MeshMaterials.Num() > 0 )
			{
				// Note: there is no way to know which material to gather parameter names from if there is more than one.  Assume the first material
				MaterialInterface = MeshMaterialModule->MeshMaterials[0];
			}
		}

		if (MaterialInterface)
		{
			UpdateParameterNames(MaterialInterface);
			for (int32 ParamIndex = 0; ParamIndex < 4; ParamIndex++)
			{
				FString TempName = FString::Printf(TEXT("%s (DP%d)"), 
					*(DynamicParams[ParamIndex].ParamName.ToString()), ParamIndex);
				EdSetup->ChangeCurveName(
					DynamicParams[ParamIndex].ParamValue.Distribution, 
					TempName);
			}
		}
	}
#endif
}

void UParticleModuleParameterDynamic::UpdateUsageFlags()
{
	if (FPlatformProperties::HasEditorOnlyData())
	{
		bUsesVelocity = false;
		UpdateFlags = EDPU_UPDATE_ALL;
		for (int32 Index = 0; Index < 4; Index++)
		{
			FEmitterDynamicParameter& DynParam = DynamicParams[Index];
			if (DynParam.bSpawnTimeOnly == true)
			{
				UpdateFlags &= ~(1 << Index);
			}
			if ((DynParam.ValueMethod != EDPV_UserSet && DynParam.ValueMethod != EDPV_AutoSet) || 
				(DynParam.bScaleVelocityByParamValue == true))
			{
				bUsesVelocity = true;
			}
		}

		// If it is none of the specially handled cases, see if there is a way to make it one...
		if (
			(UpdateFlags != EDPU_UPDATE_0) && (UpdateFlags != EDPU_UPDATE_1) && (UpdateFlags != EDPU_UPDATE_2) && (UpdateFlags != EDPU_UPDATE_3) && 
			(UpdateFlags != EDPU_UPDATE_01) && (UpdateFlags != EDPU_UPDATE_012) && (UpdateFlags != EDPU_UPDATE_ALL) && (UpdateFlags != EDPU_UPDATE_NONE)
			)
		{
			// See if any of the ones set to not update are constant
			for (int32 Index = 0; Index < 4; Index++)
			{
				FEmitterDynamicParameter& DynParam = DynamicParams[Index];
				if ((DynParam.bSpawnTimeOnly == true) && (DynParam.bScaleVelocityByParamValue == EDPV_UserSet))
				{
					// See what the distribution is...
					UDistributionFloatConstant* DistConstant = Cast<UDistributionFloatConstant>(DynParam.ParamValue.Distribution);
					if (DistConstant != NULL)
					{
						if (Index == 3)
						{
							if (UpdateFlags == EDPU_UPDATE_012)
							{
								// Don't bother setting it in this case as '012' is slightly faster than all
								continue;
							}
						}
						// It's constant, spawn-time only so it is safe to always update it.
						UpdateFlags &= (1 << Index);
					}
				}
			}
		}
	}
}

/*-----------------------------------------------------------------------------
	UParticleModuleParameterDynamic_Seeded implementation.
-----------------------------------------------------------------------------*/
UParticleModuleParameterDynamic_Seeded::UParticleModuleParameterDynamic_Seeded(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bSpawnModule = true;
	bUpdateModule = true;
	bSupportsRandomSeed = true;
	bRequiresLoopingNotification = true;
}

void UParticleModuleParameterDynamic_Seeded::EmitterLoopingNotify(FParticleEmitterInstance* Owner)
{
	if (RandomSeedInfo.bResetSeedOnEmitterLooping == true)
	{
		FParticleRandomSeedInstancePayload* Payload = (FParticleRandomSeedInstancePayload*)(Owner->GetModuleInstanceData(this));
		PrepRandomSeedInstancePayload(Owner, Payload, RandomSeedInfo);
	}
}
