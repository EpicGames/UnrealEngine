// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	ParticleModules_Size.cpp: 
	Size-related particle module implementations.
=============================================================================*/

#include "ParticleEmitterInstances.h"
#include "Particles/ParticleSystemComponent.h"
#include "Distributions/DistributionVectorConstant.h"
#include "Distributions/DistributionVectorUniform.h"
#include "Distributions/DistributionVectorConstantCurve.h"
#include "Particles/ParticleModule.h"
#include "Particles/Size/ParticleModuleSizeBase.h"
#include "Particles/Size/ParticleModuleSize.h"
#include "Particles/Size/ParticleModuleSize_Seeded.h"
#include "Particles/Size/ParticleModuleSizeMultiplyLife.h"
#include "Particles/Size/ParticleModuleSizeScale.h"
#include "Particles/Size/ParticleModuleSizeScaleBySpeed.h"
#include "Particles/TypeData/ParticleModuleTypeDataBase.h"
#include "Particles/TypeData/ParticleModuleTypeDataGpu.h"
#include "Particles/ParticleLODLevel.h"
#include "Particles/ParticleModuleRequired.h"

UParticleModuleSizeBase::UParticleModuleSizeBase(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

/*-----------------------------------------------------------------------------
	Abstract base modules used for categorization.
-----------------------------------------------------------------------------*/


/*-----------------------------------------------------------------------------
	UParticleModuleSize implementation.
-----------------------------------------------------------------------------*/

UParticleModuleSize::UParticleModuleSize(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bSpawnModule = true;
	bUpdateModule = false;
}

void UParticleModuleSize::InitializeDefaults()
{
	if (!StartSize.IsCreated())
	{
		UDistributionVectorUniform* DistributionStartSize = NewObject<UDistributionVectorUniform>(this, TEXT("DistributionStartSize"));
		DistributionStartSize->Min = FVector(1.0f, 1.0f, 1.0f);
		DistributionStartSize->Max = FVector(1.0f, 1.0f, 1.0f);
		StartSize.Distribution = DistributionStartSize;
	}
}

void UParticleModuleSize::PostInitProperties()
{
	Super::PostInitProperties();
	if (!HasAnyFlags(RF_ClassDefaultObject | RF_NeedLoad))
	{
		InitializeDefaults();
	}
}

void UParticleModuleSize::CompileModule( FParticleEmitterBuildInfo& EmitterInfo )
{
	float MinSize = 0.0f;
	float MaxSize = 0.0f;
	StartSize.GetValue();
	StartSize.GetOutRange( MinSize, MaxSize );
	EmitterInfo.MaxSize.X *= MaxSize;
	EmitterInfo.MaxSize.Y *= MaxSize;
	EmitterInfo.SpawnModules.Add( this );
	EmitterInfo.SizeScale.InitializeWithConstant( FVector( 1.0f, 1.0f, 1.0f ) );
}

#if WITH_EDITOR
void UParticleModuleSize::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	InitializeDefaults();
	Super::PostEditChangeProperty(PropertyChangedEvent);
}
#endif // WITH_EDITOR

void UParticleModuleSize::Spawn(const FSpawnContext& Context)
{
	SpawnEx(Context, &GetRandomStream(Context));
}

void UParticleModuleSize::SpawnEx(const FSpawnContext& Context, struct FRandomStream* InRandomStream)
{
	FParticleEmitterInstance* Owner = &Context.Owner;

	SPAWN_INIT;
	FVector Size		 = StartSize.GetValue(Owner->EmitterTime, Context.GetDistributionData(), 0, InRandomStream);
	Particle.Size	+= (FVector3f)Size;

	AdjustParticleBaseSizeForUVFlipping(Size, Owner->CurrentLODLevel->RequiredModule->UVFlippingMode, *InRandomStream);
	Particle.BaseSize += (FVector3f)Size;
}

/*-----------------------------------------------------------------------------
	UParticleModuleSize_Seeded implementation.
-----------------------------------------------------------------------------*/
UParticleModuleSize_Seeded::UParticleModuleSize_Seeded(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bSpawnModule = true;
	bSupportsRandomSeed = true;
	bRequiresLoopingNotification = true;
}

void UParticleModuleSize_Seeded::EmitterLoopingNotify(FParticleEmitterInstance* Owner)
{
	if (RandomSeedInfo.bResetSeedOnEmitterLooping == true)
	{
		FParticleRandomSeedInstancePayload* Payload = Owner->GetModuleRandomSeedInstanceData(this);
		PrepRandomSeedInstancePayload(Owner, Payload, RandomSeedInfo);
	}
}

/*-----------------------------------------------------------------------------
	UParticleModuleSizeMultiplyLife implementation.
-----------------------------------------------------------------------------*/
UParticleModuleSizeMultiplyLife::UParticleModuleSizeMultiplyLife(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bSpawnModule = true;
	bUpdateModule = true;
	MultiplyX = true;
	MultiplyY = true;
	MultiplyZ = true;
}

void UParticleModuleSizeMultiplyLife::InitializeDefaults()
{
	if (!LifeMultiplier.IsCreated())
	{
		LifeMultiplier.Distribution = NewObject<UDistributionVectorConstant>(this, TEXT("DistributionLifeMultiplier"));
	}
}

void UParticleModuleSizeMultiplyLife::PostInitProperties()
{
	Super::PostInitProperties();
	if (!HasAnyFlags(RF_ClassDefaultObject | RF_NeedLoad))
	{
		InitializeDefaults();
	}
}

void UParticleModuleSizeMultiplyLife::CompileModule( FParticleEmitterBuildInfo& EmitterInfo )
{
	FVector AxisScaleMask(
		MultiplyX ? 1.0f : 0.0f,
		MultiplyY ? 1.0f : 0.0f,
		MultiplyZ ? 1.0f : 0.0f
		);
	FVector AxisKeepMask(
		1.0f - AxisScaleMask.X,
		1.0f - AxisScaleMask.Y,
		1.0f - AxisScaleMask.Z
		);
	EmitterInfo.SizeScale.Initialize( LifeMultiplier.Distribution );
	EmitterInfo.SizeScale.ScaleByConstantVector( AxisScaleMask );
	EmitterInfo.SizeScale.AddConstantVector( AxisKeepMask );
}

#if WITH_EDITOR
void UParticleModuleSizeMultiplyLife::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	InitializeDefaults();
	Super::PostEditChangeProperty(PropertyChangedEvent);
}

bool UParticleModuleSizeMultiplyLife::IsValidForLODLevel(UParticleLODLevel* LODLevel, FString& OutErrorString)
{
	if (LODLevel->TypeDataModule && LODLevel->TypeDataModule->IsA(UParticleModuleTypeDataGpu::StaticClass()))
	{
		if(!IsDistributionAllowedOnGPU(LifeMultiplier.Distribution))
		{
			OutErrorString = GetDistributionNotAllowedOnGPUText(StaticClass()->GetName(), "LifeMultiplier" ).ToString();
			return false;
		}
	}

	return true;
}

#endif // WITH_EDITOR

void UParticleModuleSizeMultiplyLife::Spawn(const FSpawnContext& Context)
{
	FParticleEmitterInstance* Owner = &Context.Owner;

	SPAWN_INIT;
	FVector SizeScale = LifeMultiplier.GetValue(Particle.RelativeTime, Context.GetDistributionData());
	if(MultiplyX)
	{
		Particle.Size.X *= SizeScale.X;
	}
	if(MultiplyY)
	{
		Particle.Size.Y *= SizeScale.Y;
	}
	if(MultiplyZ)
	{
		Particle.Size.Z *= SizeScale.Z;
	}
}

void UParticleModuleSizeMultiplyLife::Update(const FUpdateContext& Context)
{
	FParticleEmitterInstance* Owner = &Context.Owner;
	if ((Owner == NULL) || (Owner->ActiveParticles <= 0) || 
		(Owner->ParticleData == NULL) || (Owner->ParticleIndices == NULL))
	{
		return;
	}
	const FRawDistribution* FastDistribution = LifeMultiplier.GetFastRawDistribution();
	FPlatformMisc::Prefetch(Owner->ParticleData, (Owner->ParticleIndices[0] * Owner->ParticleStride));
	FPlatformMisc::Prefetch(Owner->ParticleData, (Owner->ParticleIndices[0] * Owner->ParticleStride) + PLATFORM_CACHE_LINE_SIZE);
	if (MultiplyX && MultiplyY && MultiplyZ)
	{
		if (FastDistribution)
		{
			FVector3f SizeScale;
			// fast path
			BEGIN_UPDATE_LOOP;
				FastDistribution->GetValue3None(Particle.RelativeTime, &SizeScale.X);
				FPlatformMisc::Prefetch(ParticleData, (ParticleIndices[i+1] * ParticleStride));
				FPlatformMisc::Prefetch(ParticleData, (ParticleIndices[i+1] * ParticleStride) + PLATFORM_CACHE_LINE_SIZE);
				Particle.Size.X *= SizeScale.X;
				Particle.Size.Y *= SizeScale.Y;
				Particle.Size.Z *= SizeScale.Z;
			END_UPDATE_LOOP;
		}
		else
		{
			BEGIN_UPDATE_LOOP
			{
				FVector3f SizeScale(LifeMultiplier.GetValue(Particle.RelativeTime, Context.GetDistributionData()));
				FPlatformMisc::Prefetch(ParticleData, (ParticleIndices[i+1] * ParticleStride));
				FPlatformMisc::Prefetch(ParticleData, (ParticleIndices[i+1] * ParticleStride) + PLATFORM_CACHE_LINE_SIZE);
				Particle.Size.X *= SizeScale.X;
				Particle.Size.Y *= SizeScale.Y;
				Particle.Size.Z *= SizeScale.Z;
			}
			END_UPDATE_LOOP;
		}
	}
	else
	{
		if (
			( MultiplyX && !MultiplyY && !MultiplyZ) ||
			(!MultiplyX &&  MultiplyY && !MultiplyZ) ||
			(!MultiplyX && !MultiplyY &&  MultiplyZ)
			)
		{
			int32 Index = MultiplyX ? 0 : (MultiplyY ? 1 : 2);
			BEGIN_UPDATE_LOOP
			{
				FVector SizeScale = LifeMultiplier.GetValue(Particle.RelativeTime, Context.GetDistributionData());
				FPlatformMisc::Prefetch(ParticleData, (ParticleIndices[i+1] * ParticleStride));
				FPlatformMisc::Prefetch(ParticleData, (ParticleIndices[i+1] * ParticleStride) + PLATFORM_CACHE_LINE_SIZE);
				Particle.Size[Index] *= SizeScale[Index];
			}
			END_UPDATE_LOOP;
		}
		else
		{
			BEGIN_UPDATE_LOOP
			{
				FVector3f SizeScale(LifeMultiplier.GetValue(Particle.RelativeTime, Context.GetDistributionData()));
				FPlatformMisc::Prefetch(ParticleData, (ParticleIndices[i+1] * ParticleStride));
				FPlatformMisc::Prefetch(ParticleData, (ParticleIndices[i+1] * ParticleStride) + PLATFORM_CACHE_LINE_SIZE);
				if(MultiplyX)
				{
					Particle.Size.X *= SizeScale.X;
				}
				if(MultiplyY)
				{
					Particle.Size.Y *= SizeScale.Y;
				}
				if(MultiplyZ)
				{
					Particle.Size.Z *= SizeScale.Z;
				}
			}
			END_UPDATE_LOOP;
		}
	}
}

void UParticleModuleSizeMultiplyLife::SetToSensibleDefaults(UParticleEmitter* Owner)
{
	LifeMultiplier.Distribution = NewObject<UDistributionVectorConstantCurve>(this);
	UDistributionVectorConstantCurve* LifeMultiplierDist = Cast<UDistributionVectorConstantCurve>(LifeMultiplier.Distribution);
	if (LifeMultiplierDist)
	{
		// Add two points, one at time 0.0f and one at 1.0f
		for (int32 Key = 0; Key < 2; Key++)
		{
			int32	KeyIndex = LifeMultiplierDist->CreateNewKey(Key * 1.0f);
			for (int32 SubIndex = 0; SubIndex < 3; SubIndex++)
			{
				LifeMultiplierDist->SetKeyOut(SubIndex, KeyIndex, 1.0f);
			}
		}
		LifeMultiplierDist->bIsDirty = true;
	}
}


/*-----------------------------------------------------------------------------
	UParticleModuleSizeScale implementation.
-----------------------------------------------------------------------------*/
UParticleModuleSizeScale::UParticleModuleSizeScale(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bSpawnModule = true;
	bUpdateModule = true;
	EnableX = true;
	EnableY = true;
	EnableZ = true;
}

void UParticleModuleSizeScale::InitializeDefaults()
{
	if (!SizeScale.IsCreated())
	{
		SizeScale.Distribution = NewObject<UDistributionVectorConstant>(this, TEXT("DistributionSizeScale"));
	}
}

void UParticleModuleSizeScale::PostInitProperties()
{
	Super::PostInitProperties();
	if (!HasAnyFlags(RF_ClassDefaultObject | RF_NeedLoad))
	{
		InitializeDefaults();
	}
}

void UParticleModuleSizeScale::CompileModule( FParticleEmitterBuildInfo& EmitterInfo )
{
	EmitterInfo.SizeScale.Initialize( SizeScale.Distribution );
}

#if WITH_EDITOR
void UParticleModuleSizeScale::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	InitializeDefaults();
	Super::PostEditChangeProperty(PropertyChangedEvent);
}

bool UParticleModuleSizeScale::IsValidForLODLevel(UParticleLODLevel* LODLevel, FString& OutErrorString)
{
	if (LODLevel->TypeDataModule && LODLevel->TypeDataModule->IsA(UParticleModuleTypeDataGpu::StaticClass()))
	{
		if(!IsDistributionAllowedOnGPU(SizeScale.Distribution))
		{
			OutErrorString = GetDistributionNotAllowedOnGPUText(StaticClass()->GetName(), "SizeScale" ).ToString();
			return false;
		}
	}

	return true;
}

#endif // WITH_EDITOR

void UParticleModuleSizeScale::Spawn(const FSpawnContext& Context)
{
	FParticleEmitterInstance* Owner = &Context.Owner;
	SPAWN_INIT;
	FVector3f ScaleFactor(SizeScale.GetValue(Particle.RelativeTime, Context.GetDistributionData()));
	Particle.Size = GetParticleBaseSize(*ParticleBase) * ScaleFactor;
}

void UParticleModuleSizeScale::Update(const FUpdateContext& Context)
{
	FParticleEmitterInstance* Owner = &Context.Owner;
	BEGIN_UPDATE_LOOP;
		FVector3f ScaleFactor(SizeScale.GetValue(Particle.RelativeTime, Context.GetDistributionData()));
		Particle.Size = GetParticleBaseSize(Particle) * ScaleFactor;
	END_UPDATE_LOOP;
}

void UParticleModuleSizeScale::SetToSensibleDefaults(UParticleEmitter* Owner)
{
	UDistributionVectorConstant* SizeScaleDist = Cast<UDistributionVectorConstant>(SizeScale.Distribution);
	if (SizeScaleDist)
	{
		SizeScaleDist->Constant = FVector(1.0f,1.0f,1.0f);
		SizeScaleDist->bIsDirty = true;
	}
}

/*------------------------------------------------------------------------------
	Scale size by speed module.
------------------------------------------------------------------------------*/
UParticleModuleSizeScaleBySpeed::UParticleModuleSizeScaleBySpeed(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bUpdateModule = true;
	MaxScale.X = 1;
	MaxScale.Y = 1;
}

void UParticleModuleSizeScaleBySpeed::Update(const FUpdateContext& Context)
{
	FVector Scale(SpeedScale.X, SpeedScale.Y, 1.0f);
	FVector ScaleMax(MaxScale.X, MaxScale.Y, 1.0f);

	FParticleEmitterInstance* Owner = &Context.Owner;
	BEGIN_UPDATE_LOOP;
		FVector Size = Scale * Particle.Velocity.Size();
		Size = Size.ComponentMax(FVector(1.0f));
		Size = Size.ComponentMin(ScaleMax);
		Particle.Size = GetParticleBaseSize(Particle) * (FVector3f)Size;
	END_UPDATE_LOOP;
}


/**
 * Compile the effects of this module on runtime simulation.
 * @param EmitterInfo - Information needed for runtime simulation.
 */
void UParticleModuleSizeScaleBySpeed::CompileModule(FParticleEmitterBuildInfo& EmitterInfo)
{
	EmitterInfo.SizeScaleBySpeed = SpeedScale;
	EmitterInfo.MaxSizeScaleBySpeed = MaxScale;
}
