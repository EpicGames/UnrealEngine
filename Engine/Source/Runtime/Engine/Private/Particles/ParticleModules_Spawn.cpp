// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	ParticleModules_Spawn.cpp: Particle spawn-related module implementations.
=============================================================================*/

#include "ParticleEmitterInstances.h"
#include "Particles/ParticleSystemComponent.h"
#include "Distributions/DistributionFloatConstant.h"
#include "Particles/ParticleEmitter.h"
#include "Particles/Spawn/ParticleModuleSpawnBase.h"
#include "Particles/ParticleModule.h"
#include "Particles/Spawn/ParticleModuleSpawn.h"
#include "Particles/Spawn/ParticleModuleSpawnPerUnit.h"
#include "Distributions/DistributionFloatConstantCurve.h"

UParticleModuleSpawnBase::UParticleModuleSpawnBase(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bProcessSpawnRate = true;
	bProcessBurstList = true;
}

/*-----------------------------------------------------------------------------
	Abstract base modules used for categorization.
-----------------------------------------------------------------------------*/

/*-----------------------------------------------------------------------------
	UParticleModuleSpawn implementation.
-----------------------------------------------------------------------------*/
UParticleModuleSpawn::UParticleModuleSpawn(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bProcessSpawnRate = true;
	LODDuplicate = false;
	bApplyGlobalSpawnRateScale = true;
}

void UParticleModuleSpawn::InitializeDefaults()
{
	if(!Rate.IsCreated())
	{
		UDistributionFloatConstant* RequiredDistributionSpawnRate = NewObject<UDistributionFloatConstant>(this, TEXT("RequiredDistributionSpawnRate"));
		RequiredDistributionSpawnRate->Constant = 20.0f;
		Rate.Distribution = RequiredDistributionSpawnRate;
	}

	if(!RateScale.IsCreated())
	{
		UDistributionFloatConstant* RequiredDistributionSpawnRateScale = NewObject<UDistributionFloatConstant>(this, TEXT("RequiredDistributionSpawnRateScale"));
		RequiredDistributionSpawnRateScale->Constant = 1.0f;
		RateScale.Distribution = RequiredDistributionSpawnRateScale;
	}

	if(!BurstScale.IsCreated())
	{
		UDistributionFloatConstant* BurstScaleDistribution = NewObject<UDistributionFloatConstant>(this, TEXT("BurstScaleDistribution"));
		BurstScaleDistribution->Constant = 1.0f;
		BurstScale.Distribution = BurstScaleDistribution;
	}
}

void UParticleModuleSpawn::PostInitProperties()
{
	Super::PostInitProperties();
	if (!HasAnyFlags(RF_ClassDefaultObject | RF_NeedLoad))
	{
		InitializeDefaults();
	}
}

void UParticleModuleSpawn::PostLoad()
{
	Super::PostLoad();

	if (GetLinkerUEVersion() < VER_UE4_GLOBAL_EMITTER_SPAWN_RATE_SCALE && Rate.Distribution)
	{
		Rate.Distribution->ConditionalPostLoad();//Ensure the distribution is loaded.
		if (Rate.Distribution->IsA(UDistributionFloatConstant::StaticClass()) && Rate.GetValue() <= 1.0f)
		{
			bApplyGlobalSpawnRateScale = false;
		}
	}
}

#if WITH_EDITOR
void UParticleModuleSpawn::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	InitializeDefaults();

	for (int32 Index = 0; Index < BurstList.Num(); Index++)
	{
		FParticleBurst* Burst = &(BurstList[Index]);

		// Clamp them to positive numbers...
		Burst->Count = FMath::Max<int32>(0, Burst->Count);
		if (Burst->CountLow > -1)
		{
			Burst->CountLow = FMath::Min<int32>(Burst->Count, Burst->CountLow);
		}
	}

	Super::PostEditChangeProperty(PropertyChangedEvent);
}
#endif // WITH_EDITOR

bool UParticleModuleSpawn::GetSpawnAmount(const FContext& Context, 
	int32 Offset, float OldLeftover, float DeltaTime, int32& Number, float& InRate)
{
	return false;
}

bool UParticleModuleSpawn::GenerateLODModuleValues(UParticleModule* SourceModule, float Percentage, UParticleLODLevel* LODLevel)
{
	// Convert the module values
	UParticleModuleSpawn* SpawnSource = Cast<UParticleModuleSpawn>(SourceModule);
	if (!SpawnSource)
	{
		return false;
	}

	bool bResult	= true;
	if (FPlatformProperties::HasEditorOnlyData())
	{
		//SpawnRate
		// TODO: Make sure these functions are never called on console, or when the UDistributions are missing
		if (ConvertFloatDistribution(Rate.Distribution, SpawnSource->Rate.Distribution, Percentage) == false)
		{
			bResult	= false;
		}

		//ParticleBurstMethod
		//BurstList
		check(BurstList.Num() == SpawnSource->BurstList.Num());
		for (int32 BurstIndex = 0; BurstIndex < SpawnSource->BurstList.Num(); BurstIndex++)
		{
			FParticleBurst* SourceBurst	= &(SpawnSource->BurstList[BurstIndex]);
			FParticleBurst* Burst		= &(BurstList[BurstIndex]);

			Burst->Time = SourceBurst->Time;
			// Don't drop below 1...
			if (Burst->Count > 0)
			{
				Burst->Count = FMath::TruncToInt(SourceBurst->Count * (Percentage / 100.0f));
				if (Burst->Count == 0)
				{
					Burst->Count = 1;
				}
			}
		}
	}
	return bResult;
}

float UParticleModuleSpawn::GetMaximumSpawnRate()
{
	float MinSpawn, MaxSpawn;
	float MinScale, MaxScale;

	Rate.GetOutRange(MinSpawn, MaxSpawn);
	RateScale.GetOutRange(MinScale, MaxScale);

	return (MaxSpawn * MaxScale);
}

float UParticleModuleSpawn::GetEstimatedSpawnRate()
{
	float MinSpawn, MaxSpawn;
	float MinScale, MaxScale;

	Rate.GetOutRange(MinSpawn, MaxSpawn);
	RateScale.GetOutRange(MinScale, MaxScale);

	UDistributionFloatConstantCurve* RateScaleCurve = Cast<UDistributionFloatConstantCurve>(RateScale.Distribution);
	if (RateScaleCurve != NULL)
	{
		// We need to walk the curve and determine the average
		int32 KeyCount = RateScaleCurve->GetNumKeys();
		if (KeyCount > 1)
		{
			float SummedAverage = 0.0f;
			float LastKeyIn = RateScaleCurve->GetKeyIn(KeyCount - 1);
			float PrevKeyIn = FMath::Max<float>(0.0f, RateScaleCurve->GetKeyIn(0));
			float TotalTime = FMath::Max<float>(1.0f, LastKeyIn - PrevKeyIn);
			float PrevKeyOut = RateScaleCurve->GetKeyOut(0, 0);
			for (int32 KeyIndex = 1; KeyIndex < KeyCount; KeyIndex++)
			{
				float KeyIn = RateScaleCurve->GetKeyIn(KeyIndex);
				float KeyOut = RateScaleCurve->GetKeyOut(0, KeyIndex);

				float Delta = (KeyIn - PrevKeyIn) / TotalTime;
				float Avg = (KeyOut + PrevKeyOut) / 2.0f;
				SummedAverage += Delta * Avg;

				PrevKeyIn = KeyIn;
				PrevKeyOut = KeyOut;
			}

			MaxScale = SummedAverage;
		}
	}

	// We need to estimate the value for curves to prevent short spikes from inflating the value... 
	UDistributionFloatConstantCurve* RateCurve = Cast<UDistributionFloatConstantCurve>(Rate.Distribution);
	if (RateCurve != NULL)
	{
		// We need to walk the curve and determine the average
		int32 KeyCount = RateCurve->GetNumKeys();
		if (KeyCount > 1)
		{
			float SummedAverage = 0.0f;
			float LastKeyIn = RateCurve->GetKeyIn(KeyCount - 1);
			float PrevKeyIn = FMath::Max<float>(0.0f, RateCurve->GetKeyIn(0));
			float TotalTime = FMath::Max<float>(1.0f, LastKeyIn - PrevKeyIn);
			float PrevKeyOut = RateCurve->GetKeyOut(0, 0);
			for (int32 KeyIndex = 1; KeyIndex < KeyCount; KeyIndex++)
			{
				float KeyIn = RateCurve->GetKeyIn(KeyIndex);
				float KeyOut = RateCurve->GetKeyOut(0, KeyIndex);

				float Delta = (KeyIn - PrevKeyIn) / TotalTime;
				float Avg = ((KeyOut + PrevKeyOut) * MaxScale) / 2.0f;
				SummedAverage += Delta * Avg;

				PrevKeyIn = KeyIn;
				PrevKeyOut = KeyOut;
			}

			MaxSpawn = SummedAverage;// / (FMath::Max<float>(1.0f, LastKeyIn));
			return MaxSpawn;
		}
	}

	return (MaxSpawn * MaxScale);
}

int32 UParticleModuleSpawn::GetMaximumBurstCount()
{
	// Note that this does not take into account entries could be outside of the emitter duration!
	int32 MaxBurst = 0;
	for (int32 BurstIndex = 0; BurstIndex < BurstList.Num(); BurstIndex++)
	{
		MaxBurst += BurstList[BurstIndex].Count;
	}
	return MaxBurst;
}

float UParticleModuleSpawn::GetGlobalRateScale()const
{
	static const auto EmitterRateScaleCVar = IConsoleManager::Get().FindTConsoleVariableDataFloat(TEXT("r.EmitterSpawnRateScale"));
	return (bApplyGlobalSpawnRateScale && EmitterRateScaleCVar) ? EmitterRateScaleCVar->GetValueOnAnyThread() : 1.0f;
}

/*-----------------------------------------------------------------------------
	UParticleModuleSpawnPerUnit implementation.
-----------------------------------------------------------------------------*/
UParticleModuleSpawnPerUnit::UParticleModuleSpawnPerUnit(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bSpawnModule = false;
	bUpdateModule = false;
	UnitScalar = 50.0f;
	MovementTolerance = 0.1f;
}

void UParticleModuleSpawnPerUnit::InitializeDefaults()
{
	if (!SpawnPerUnit.IsCreated())
	{
		UDistributionFloatConstant* RequiredDistributionSpawnPerUnit = NewObject<UDistributionFloatConstant>(this, TEXT("RequiredDistributionSpawnPerUnit"));
		RequiredDistributionSpawnPerUnit->Constant = 0.0f;
		SpawnPerUnit.Distribution = RequiredDistributionSpawnPerUnit;
	}
}

void UParticleModuleSpawnPerUnit::PostInitProperties()
{
	Super::PostInitProperties();
	if (!HasAnyFlags(RF_ClassDefaultObject | RF_NeedLoad))
	{
		InitializeDefaults();
	}
}

void UParticleModuleSpawnPerUnit::CompileModule( FParticleEmitterBuildInfo& EmitterInfo )
{
	EmitterInfo.SpawnPerUnitModule = this;
}

#if WITH_EDITOR
void UParticleModuleSpawnPerUnit::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	InitializeDefaults();
	Super::PostEditChangeProperty(PropertyChangedEvent);
}
#endif // WITH_EDITOR

uint32 UParticleModuleSpawnPerUnit::RequiredBytesPerInstance()
{
	return sizeof(FParticleSpawnPerUnitInstancePayload);
}

bool UParticleModuleSpawnPerUnit::GetSpawnAmount(const FContext& Context, 
	int32 Offset, float OldLeftover, float DeltaTime, int32& Number, float& Rate)
{
	FParticleEmitterInstance* Owner = &Context.Owner;
	check(Owner);

	bool bMoved = false;
	FParticleSpawnPerUnitInstancePayload* SPUPayload = NULL;
	float NewTravelLeftover = 0.0f;

	float ParticlesPerUnit = SpawnPerUnit.GetValue(Owner->EmitterTime, Context.GetDistributionData()) / UnitScalar;
	// Allow for PPU of 0.0f to allow for 'turning off' an emitter when moving
	if (ParticlesPerUnit >= 0.0f)
	{
		float LeftoverTravel = 0.0f;
		uint8* InstData = Owner->GetModuleInstanceData(this);
		if (InstData)
		{
			SPUPayload = (FParticleSpawnPerUnitInstancePayload*)InstData;
			LeftoverTravel = SPUPayload->CurrentDistanceTravelled;
		}

		// Calculate movement delta over last frame, include previous remaining delta
		FVector TravelDirection = Owner->Location - Owner->OldLocation;
		FVector RemoveComponentMultiplier(
			bIgnoreMovementAlongX ? 0.0f : 1.0f,
			bIgnoreMovementAlongY ? 0.0f : 1.0f,
			bIgnoreMovementAlongZ ? 0.0f : 1.0f
			);
		TravelDirection *= RemoveComponentMultiplier;

		// Calculate distance traveled
		float TravelDistance = TravelDirection.Size();
		if (MaxFrameDistance > 0.0f)
		{
			if (TravelDistance > MaxFrameDistance)
			{
				// Clear it out!
				//@todo. Need to 'shift' the start point closer so we can still spawn...
				TravelDistance = 0.0f;
				if ( SPUPayload )
				{
					SPUPayload->CurrentDistanceTravelled = 0.0f;
				}
			}
		}

		if (TravelDistance > 0.0f)
		{
			if (TravelDistance > (MovementTolerance * UnitScalar))
			{
				bMoved = true;
			}

			// Normalize direction for use later
			TravelDirection.Normalize();

			// Calculate number of particles to emit
			float NewLeftover = (TravelDistance + LeftoverTravel) * ParticlesPerUnit;
			Number = FMath::FloorToInt(NewLeftover);
			float InvDeltaTime = (DeltaTime > 0.0f) ? 1.0f / DeltaTime : 0.0f;
			Rate = Number * InvDeltaTime;
			NewTravelLeftover = (TravelDistance + LeftoverTravel) - (Number * UnitScalar);
			if (SPUPayload)
			{
				SPUPayload->CurrentDistanceTravelled = FMath::Max<float>(0.0f, NewTravelLeftover);
			}

		}
		else
		{
			Number = 0;
			Rate = 0.0f;
		}
	}
	else
	{
		Number = 0;
		Rate = 0.0f;
	}

	if (bIgnoreSpawnRateWhenMoving == true)
	{
		if (bMoved == true)
		{
			return false;
		}
		return true;
	}

	return bProcessSpawnRate;
}
