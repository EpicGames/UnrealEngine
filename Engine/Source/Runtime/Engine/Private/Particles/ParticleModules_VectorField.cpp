// Copyright Epic Games, Inc. All Rights Reserved.

/*==============================================================================
	ParticleModules_VectorField.cpp: Vector field module implementations.
==============================================================================*/

#include "Distributions/DistributionFloatConstant.h"
#include "ParticleEmitterInstances.h"
#include "Particles/TypeData/ParticleModuleTypeDataGpu.h"
#include "Particles/ParticleModule.h"
#include "Particles/VectorField/ParticleModuleVectorFieldBase.h"
#include "Particles/TypeData/ParticleModuleTypeDataBase.h"
#include "Particles/VectorField/ParticleModuleVectorFieldLocal.h"
#include "Particles/VectorField/ParticleModuleVectorFieldGlobal.h"
#include "Particles/VectorField/ParticleModuleVectorFieldRotation.h"
#include "Particles/VectorField/ParticleModuleVectorFieldRotationRate.h"
#include "Particles/VectorField/ParticleModuleVectorFieldScale.h"
#include "Particles/VectorField/ParticleModuleVectorFieldScaleOverLife.h"
#include "Particles/ParticleLODLevel.h"

/*------------------------------------------------------------------------------
	Global vector field scale.
------------------------------------------------------------------------------*/
UParticleModuleVectorFieldBase::UParticleModuleVectorFieldBase(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

UParticleModuleVectorFieldGlobal::UParticleModuleVectorFieldGlobal(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

void UParticleModuleVectorFieldGlobal::CompileModule(FParticleEmitterBuildInfo& EmitterInfo)
{
	EmitterInfo.GlobalVectorFieldScale = GlobalVectorFieldScale;
	EmitterInfo.GlobalVectorFieldTightness = GlobalVectorFieldTightness;
}

/*------------------------------------------------------------------------------
	Per-particle vector field scale.
------------------------------------------------------------------------------*/
UParticleModuleVectorFieldScale::UParticleModuleVectorFieldScale(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

void UParticleModuleVectorFieldScale::InitializeDefaults()
{
	if (!VectorFieldScaleRaw.IsCreated())
	{
		UDistributionFloatConstant* DistributionVectorFieldScale = NewObject<UDistributionFloatConstant>(this, TEXT("DistributionVectorFieldScale"));
		DistributionVectorFieldScale->Constant = 1.0f;
		VectorFieldScaleRaw.Distribution = DistributionVectorFieldScale;
	}
}
void UParticleModuleVectorFieldScale::PostInitProperties()
{
	Super::PostInitProperties();
	if (!HasAnyFlags(RF_ClassDefaultObject | RF_NeedLoad))
	{
		InitializeDefaults();
	}
}

void UParticleModuleVectorFieldScale::PostLoad()
{
	Super::PostLoad();
#if WITH_EDITOR
	if (VectorFieldScale_DEPRECATED)
	{
		VectorFieldScaleRaw.Distribution = VectorFieldScale_DEPRECATED;
		VectorFieldScaleRaw.Initialize();
	}
	VectorFieldScale_DEPRECATED = nullptr;
#endif
}

#if WITH_EDITOR
void UParticleModuleVectorFieldScale::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	InitializeDefaults();
	Super::PostEditChangeProperty(PropertyChangedEvent);
}
#endif // WITH_EDITOR

void UParticleModuleVectorFieldScale::CompileModule(FParticleEmitterBuildInfo& EmitterInfo)
{
	EmitterInfo.VectorFieldScale.ScaleByDistribution(VectorFieldScaleRaw.Distribution);
}

#if WITH_EDITOR

bool UParticleModuleVectorFieldScale::IsValidForLODLevel(UParticleLODLevel* LODLevel, FString& OutErrorString)
{
	if (LODLevel->TypeDataModule && LODLevel->TypeDataModule->IsA(UParticleModuleTypeDataGpu::StaticClass()))
	{
		if(!IsDistributionAllowedOnGPU(VectorFieldScaleRaw.Distribution))
		{
			OutErrorString = GetDistributionNotAllowedOnGPUText(StaticClass()->GetName(), "VectorFieldScale" ).ToString();
			return false;
		}
	}

	return true;
}

#endif
/*------------------------------------------------------------------------------
	Per-particle vector field scale over life.
------------------------------------------------------------------------------*/
UParticleModuleVectorFieldScaleOverLife::UParticleModuleVectorFieldScaleOverLife(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

void UParticleModuleVectorFieldScaleOverLife::InitializeDefaults()
{
	if (!VectorFieldScaleOverLifeRaw.IsCreated())
	{
		UDistributionFloatConstant* DistributionVectorFieldScaleOverLife = NewObject<UDistributionFloatConstant>(this, TEXT("DistributionVectorFieldScaleOverLife"));
		DistributionVectorFieldScaleOverLife->Constant = 1.0f;
		VectorFieldScaleOverLifeRaw.Distribution = DistributionVectorFieldScaleOverLife;
	}
}

void UParticleModuleVectorFieldScaleOverLife::PostInitProperties()
{
	Super::PostInitProperties();
	if (!HasAnyFlags(RF_ClassDefaultObject | RF_NeedLoad))
	{
		InitializeDefaults();
	}
}

void UParticleModuleVectorFieldScaleOverLife::PostLoad()
{
	Super::PostLoad();
#if WITH_EDITOR
	if (VectorFieldScaleOverLife_DEPRECATED)
	{
		VectorFieldScaleOverLifeRaw.Distribution = VectorFieldScaleOverLife_DEPRECATED;
		VectorFieldScaleOverLifeRaw.Initialize();
	}
	VectorFieldScaleOverLife_DEPRECATED = nullptr;
#endif
}

#if WITH_EDITOR
void UParticleModuleVectorFieldScaleOverLife::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	InitializeDefaults();
	Super::PostEditChangeProperty(PropertyChangedEvent);
}
#endif // WITH_EDITOR

void UParticleModuleVectorFieldScaleOverLife::CompileModule(FParticleEmitterBuildInfo& EmitterInfo)
{
	EmitterInfo.VectorFieldScaleOverLife.ScaleByDistribution(VectorFieldScaleOverLifeRaw.Distribution);
}

#if WITH_EDITOR

bool UParticleModuleVectorFieldScaleOverLife::IsValidForLODLevel(UParticleLODLevel* LODLevel, FString& OutErrorString)
{
	if (LODLevel->TypeDataModule && LODLevel->TypeDataModule->IsA(UParticleModuleTypeDataGpu::StaticClass()))
	{
		if(!IsDistributionAllowedOnGPU(VectorFieldScaleOverLifeRaw.Distribution))
		{
			OutErrorString = GetDistributionNotAllowedOnGPUText(StaticClass()->GetName(), "VectorFieldScaleOverLife" ).ToString();
			return false;
		}
	}

	return true;
}

#endif
/*------------------------------------------------------------------------------
	Local vector fields.
------------------------------------------------------------------------------*/
UParticleModuleVectorFieldLocal::UParticleModuleVectorFieldLocal(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	RelativeScale3D = FVector(1.0f, 1.0f, 1.0f);

	Intensity = 1.0;
	Tightness = 0.0;
	bUseFixDT = true;
}

void UParticleModuleVectorFieldLocal::CompileModule(FParticleEmitterBuildInfo& EmitterInfo)
{
	EmitterInfo.LocalVectorField = VectorField;
	EmitterInfo.LocalVectorFieldTransform.SetTranslation(RelativeTranslation);
	EmitterInfo.LocalVectorFieldTransform.SetRotation(RelativeRotation.Quaternion());
	EmitterInfo.LocalVectorFieldTransform.SetScale3D(RelativeScale3D);
	EmitterInfo.LocalVectorFieldIntensity = Intensity;
	EmitterInfo.LocalVectorFieldTightness = Tightness;
	EmitterInfo.bLocalVectorFieldIgnoreComponentTransform = bIgnoreComponentTransform;
	EmitterInfo.bLocalVectorFieldTileX = bTileX;
	EmitterInfo.bLocalVectorFieldTileY = bTileY;
	EmitterInfo.bLocalVectorFieldTileZ = bTileZ;
	EmitterInfo.bLocalVectorFieldUseFixDT = bUseFixDT;
}

/*------------------------------------------------------------------------------
	Local vector initial rotation.
------------------------------------------------------------------------------*/
UParticleModuleVectorFieldRotation::UParticleModuleVectorFieldRotation(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

void UParticleModuleVectorFieldRotation::CompileModule(FParticleEmitterBuildInfo& EmitterInfo)
{
	EmitterInfo.LocalVectorFieldMinInitialRotation = MinInitialRotation;
	EmitterInfo.LocalVectorFieldMaxInitialRotation = MaxInitialRotation;
}

/*------------------------------------------------------------------------------
	Local vector field rotation rate.
------------------------------------------------------------------------------*/
UParticleModuleVectorFieldRotationRate::UParticleModuleVectorFieldRotationRate(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

void UParticleModuleVectorFieldRotationRate::CompileModule(FParticleEmitterBuildInfo& EmitterInfo)
{
	EmitterInfo.LocalVectorFieldRotationRate += RotationRate;
}
