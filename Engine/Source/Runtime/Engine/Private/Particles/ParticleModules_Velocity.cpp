// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	ParticleModules_Velocity.cpp: 
	Velocity-related particle module implementations.
=============================================================================*/

#include "ParticleEmitterInstances.h"
#include "ParticleEmitterInstanceOwner.h"
#include "Particles/ParticleSystemComponent.h"
#include "Distributions/DistributionFloatUniform.h"
#include "Distributions/DistributionVectorConstant.h"
#include "Distributions/DistributionVectorUniform.h"
#include "Distributions/DistributionVectorConstantCurve.h"
#include "Particles/Lifetime/ParticleModuleLifetimeBase.h"
#include "Particles/ParticleEmitter.h"
#include "Particles/Velocity/ParticleModuleVelocityBase.h"
#include "Particles/ParticleModule.h"
#include "Particles/Velocity/ParticleModuleVelocity.h"
#include "Particles/Velocity/ParticleModuleVelocityCone.h"
#include "Particles/Velocity/ParticleModuleVelocityInheritParent.h"
#include "Particles/Velocity/ParticleModuleVelocityOverLifetime.h"
#include "Particles/Velocity/ParticleModuleVelocity_Seeded.h"
#include "Particles/ParticleLODLevel.h"
#include "Particles/ParticleModuleRequired.h"
#include "PrimitiveDrawingUtils.h"

UParticleModuleVelocityBase::UParticleModuleVelocityBase(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

/*-----------------------------------------------------------------------------
	Abstract base modules used for categorization.
-----------------------------------------------------------------------------*/

/*-----------------------------------------------------------------------------
	UParticleModuleVelocity implementation.
-----------------------------------------------------------------------------*/

UParticleModuleVelocity::UParticleModuleVelocity(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bSpawnModule = true;
}

void UParticleModuleVelocity::InitializeDefaults()
{
	if (!StartVelocity.IsCreated())
	{
		StartVelocity.Distribution = NewObject<UDistributionVectorUniform>(this, TEXT("DistributionStartVelocity"));
	}

	if (!StartVelocityRadial.IsCreated())
	{
		StartVelocityRadial.Distribution = NewObject<UDistributionFloatUniform>(this, TEXT("DistributionStartVelocityRadial"));
	}
}

void UParticleModuleVelocity::PostInitProperties()
{
	Super::PostInitProperties();
	if (!HasAnyFlags(RF_ClassDefaultObject | RF_NeedLoad))
	{
		InitializeDefaults();
	}
}

#if WITH_EDITOR
void UParticleModuleVelocity::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	InitializeDefaults();
	Super::PostEditChangeProperty(PropertyChangedEvent);
}
#endif // WITH_EDITOR

void UParticleModuleVelocity::Spawn(const FSpawnContext& Context)
{
	SpawnEx(Context, &GetRandomStream(Context));
}

void UParticleModuleVelocity::SpawnEx(const FSpawnContext& Context, struct FRandomStream* InRandomStream)
{
	FParticleEmitterInstance* Owner = &Context.Owner;

	SPAWN_INIT;
	{
		FVector Vel = StartVelocity.GetValue(Owner->EmitterTime, Context.GetDistributionData(), 0, InRandomStream);
		FVector FromOrigin = (Particle.Location - Owner->EmitterToSimulation.GetOrigin()).GetSafeNormal();

		FVector OwnerScale(1.0f);
		if (bApplyOwnerScale == true)
		{
			OwnerScale = Context.GetTransform().GetScale3D();
		}

		UParticleLODLevel* LODLevel	= Owner->SpriteTemplate->GetCurrentLODLevel(Owner);
		check(LODLevel);
		if (LODLevel->RequiredModule->bUseLocalSpace)
		{
			if (bInWorldSpace == true)
			{
				Vel = Owner->SimulationToWorld.InverseTransformVector(Vel);
			}
			else
			{
				Vel = Owner->EmitterToSimulation.TransformVector(Vel);
			}
		}
		else if (bInWorldSpace == false)
		{
			Vel = Owner->EmitterToSimulation.TransformVector(Vel);
		}
		Vel *= OwnerScale;
		Vel += FromOrigin * StartVelocityRadial.GetValue(Owner->EmitterTime, Context.GetDistributionData(), InRandomStream) * OwnerScale;
		Particle.Velocity		+= (FVector3f)Vel;
		Particle.BaseVelocity	+= (FVector3f)Vel;
	}
}

/*-----------------------------------------------------------------------------
	UParticleModuleVelocityInheritParent implementation.
-----------------------------------------------------------------------------*/
UParticleModuleVelocity_Seeded::UParticleModuleVelocity_Seeded(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bSpawnModule = true;
	bSupportsRandomSeed = true;
	bRequiresLoopingNotification = true;
}

void UParticleModuleVelocity_Seeded::EmitterLoopingNotify(FParticleEmitterInstance* Owner)
{
	if (RandomSeedInfo.bResetSeedOnEmitterLooping == true)
	{
		FParticleRandomSeedInstancePayload* Payload = (FParticleRandomSeedInstancePayload*)(Owner->GetModuleInstanceData(this));
		PrepRandomSeedInstancePayload(Owner, Payload, RandomSeedInfo);
	}
}

/*-----------------------------------------------------------------------------
	UParticleModuleVelocityInheritParent implementation.
-----------------------------------------------------------------------------*/
UParticleModuleVelocityInheritParent::UParticleModuleVelocityInheritParent(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bSpawnModule = true;
}

void UParticleModuleVelocityInheritParent::InitializeDefaults()
{
	if (!Scale.IsCreated())
	{
		UDistributionVectorConstant* DistributionScale = NewObject<UDistributionVectorConstant>(this, TEXT("DistributionScale"));
		DistributionScale->Constant = FVector(1.0f, 1.0f, 1.0f);
		Scale.Distribution = DistributionScale;
	}
}

void UParticleModuleVelocityInheritParent::PostInitProperties()
{
	Super::PostInitProperties();
	if (!HasAnyFlags(RF_ClassDefaultObject | RF_NeedLoad))
	{
		InitializeDefaults();
	}
}

#if WITH_EDITOR
void UParticleModuleVelocityInheritParent::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	InitializeDefaults();
	Super::PostEditChangeProperty(PropertyChangedEvent);
}
#endif // WITH_EDITOR

void UParticleModuleVelocityInheritParent::Spawn(const FSpawnContext& Context)
{
	FParticleEmitterInstance* Owner = &Context.Owner;

	SPAWN_INIT;

	FVector Vel(0.0f);

	UParticleLODLevel* LODLevel	= Owner->SpriteTemplate->GetCurrentLODLevel(Owner);
	check(LODLevel);
	if (LODLevel->RequiredModule->bUseLocalSpace)
	{
		Vel = Context.GetTransform().InverseTransformVector(Owner->Component.GetPartSysVelocity());
	}
	else
	{
		Vel = Owner->Component.GetPartSysVelocity();
	}

	FVector vScale = Scale.GetValue(Owner->EmitterTime, Context.GetDistributionData());

	Vel *= vScale;

	Particle.Velocity		+= (FVector3f)Vel;
	Particle.BaseVelocity	+= (FVector3f)Vel;
}

/*-----------------------------------------------------------------------------
	UParticleModuleVelocityOverLifetime implementation.
-----------------------------------------------------------------------------*/
UParticleModuleVelocityOverLifetime::UParticleModuleVelocityOverLifetime(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bSpawnModule = true;
	bUpdateModule = true;

	Absolute = false;
}

void UParticleModuleVelocityOverLifetime::InitializeDefaults()
{
	if (!VelOverLife.IsCreated())
	{
		VelOverLife.Distribution = NewObject<UDistributionVectorConstantCurve>(this, TEXT("DistributionVelOverLife"));
	}
}

void UParticleModuleVelocityOverLifetime::PostInitProperties()
{
	Super::PostInitProperties();
	if (!HasAnyFlags(RF_ClassDefaultObject | RF_NeedLoad))
	{
		InitializeDefaults();
	}
}

#if WITH_EDITOR
void UParticleModuleVelocityOverLifetime::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	InitializeDefaults();
	Super::PostEditChangeProperty(PropertyChangedEvent);
}
#endif // WITH_EDITOR

void UParticleModuleVelocityOverLifetime::Spawn(const FSpawnContext& Context)
{
	if (Absolute)
	{
		SPAWN_INIT;
		FVector OwnerScale(1.0f);
		if (bApplyOwnerScale == true)
		{
			OwnerScale = Context.GetTransform().GetScale3D();
		}
		FVector Vel = VelOverLife.GetValue(Particle.RelativeTime, Context.GetDistributionData()) * OwnerScale;
		Particle.Velocity		= (FVector3f)Vel;
		Particle.BaseVelocity	= (FVector3f)Vel;
	}
}

void UParticleModuleVelocityOverLifetime::Update(const FUpdateContext& Context)
{
	FParticleEmitterInstance* Owner = &Context.Owner;
	UParticleLODLevel* LODLevel	= Owner->SpriteTemplate->GetCurrentLODLevel(Owner);
	check(LODLevel);
	FVector OwnerScale(1.0f);
	const FTransform& OwnerTM = Owner->Component.GetAsyncComponentToWorld();
	if (bApplyOwnerScale == true)
	{
		OwnerScale = OwnerTM.GetScale3D();
	}
	if (Absolute)
	{
		if (LODLevel->RequiredModule->bUseLocalSpace == false)
		{
			if (bInWorldSpace == false)
			{
				FVector Vel;
				const FMatrix LocalToWorld = OwnerTM.ToMatrixNoScale();
				BEGIN_UPDATE_LOOP;
				{
					Vel = VelOverLife.GetValue(Particle.RelativeTime, Context.GetDistributionData());
					Particle.Velocity = FVector4f(LocalToWorld.TransformVector(Vel) * OwnerScale);
				}
				END_UPDATE_LOOP;
			}
			else
			{
				BEGIN_UPDATE_LOOP;
				{
					Particle.Velocity = FVector3f(VelOverLife.GetValue(Particle.RelativeTime, Context.GetDistributionData()) * OwnerScale);
				}
				END_UPDATE_LOOP;
			}
		}
		else
		{
			if (bInWorldSpace == false)
			{
				BEGIN_UPDATE_LOOP;
				{
					Particle.Velocity = FVector3f(VelOverLife.GetValue(Particle.RelativeTime, Context.GetDistributionData()) * OwnerScale);
				}
				END_UPDATE_LOOP;
			}
			else
			{
				FVector Vel;
				const FMatrix LocalToWorld = OwnerTM.ToMatrixNoScale();
				const FMatrix InvMat = LocalToWorld.InverseFast();
				BEGIN_UPDATE_LOOP;
				{
					Vel = VelOverLife.GetValue(Particle.RelativeTime, Context.GetDistributionData());
					Particle.Velocity = FVector4f(InvMat.TransformVector(Vel) * OwnerScale);
				}
				END_UPDATE_LOOP;
			}
		}
	}
	else
	{
		if (LODLevel->RequiredModule->bUseLocalSpace == false)
		{
			FVector Vel;
			if (bInWorldSpace == false)
			{
				const FMatrix LocalToWorld = OwnerTM.ToMatrixNoScale();
				BEGIN_UPDATE_LOOP;
				{
					Vel = VelOverLife.GetValue(Particle.RelativeTime, Context.GetDistributionData());
					Particle.Velocity *= FVector4f(LocalToWorld.TransformVector(Vel) * OwnerScale);
				}
				END_UPDATE_LOOP;
			}
			else
			{
				BEGIN_UPDATE_LOOP;
				{
					Particle.Velocity *= FVector3f(VelOverLife.GetValue(Particle.RelativeTime, Context.GetDistributionData()) * OwnerScale);
				}
				END_UPDATE_LOOP;
			}
		}
		else
		{
			if (bInWorldSpace == false)
			{
				BEGIN_UPDATE_LOOP;
				{
					Particle.Velocity *= FVector3f(VelOverLife.GetValue(Particle.RelativeTime, Context.GetDistributionData()) * OwnerScale);
				}
				END_UPDATE_LOOP;
			}
			else
			{
				FVector Vel;
				const FMatrix LocalToWorld = OwnerTM.ToMatrixNoScale();
				const FMatrix InvMat = LocalToWorld.InverseFast();
				BEGIN_UPDATE_LOOP;
				{
					Vel = VelOverLife.GetValue(Particle.RelativeTime, Context.GetDistributionData());
					Particle.Velocity *= FVector4f(InvMat.TransformVector(Vel) * OwnerScale);
				}
				END_UPDATE_LOOP;
			}
		}
	}
}

/*-----------------------------------------------------------------------------
	UParticleModuleVelocityCone implementation.
-----------------------------------------------------------------------------*/
UParticleModuleVelocityCone::UParticleModuleVelocityCone(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bSpawnModule = true;
	bSupported3DDrawMode = true;
	Direction = FVector(0.0f, 0.0f, 1.0f);
}

void UParticleModuleVelocityCone::InitializeDefaults()
{
	if (!Angle.IsCreated())
	{
		Angle.Distribution = NewObject<UDistributionFloatUniform>(this, TEXT("DistributionAngle"));
	}

	if (!Velocity.IsCreated())
	{
		Velocity.Distribution = NewObject<UDistributionFloatUniform>(this, TEXT("DistributionVelocity"));
	}
}

void UParticleModuleVelocityCone::PostInitProperties()
{
	Super::PostInitProperties();
	if (!HasAnyFlags(RF_ClassDefaultObject | RF_NeedLoad))
	{
		InitializeDefaults();
	}
}

#if WITH_EDITOR
void UParticleModuleVelocityCone::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	InitializeDefaults();
	Super::PostEditChangeProperty(PropertyChangedEvent);
}
#endif // WITH_EDITOR

void UParticleModuleVelocityCone::Spawn(const FSpawnContext& Context)
{
	SpawnEx(Context, &GetRandomStream(Context));
}

void UParticleModuleVelocityCone::SpawnEx(const FSpawnContext& Context, struct FRandomStream* InRandomStream)
{
	static const float TwoPI = 2.0f * UE_PI;
	static const float ToRads = UE_PI / 180.0f;
	static const int32 UUPerRad = 10430;
	static const FVector DefaultDirection(0.0f, 0.0f, 1.0f);

	FParticleEmitterInstance* Owner = &Context.Owner;

	// Calculate the owning actor's scale
	UParticleLODLevel* LODLevel	= Owner->SpriteTemplate->GetCurrentLODLevel(Owner);
	check(LODLevel);
	FVector OwnerScale(1.0f);
	if (bApplyOwnerScale == true)
	{
		OwnerScale = Context.GetTransform().GetScale3D();
	}
	
	// Spawn particles
	SPAWN_INIT
	{
		// Calculate the default position (prior to the Direction vector being factored in)
		const float SpawnAngle = Angle.GetValue(Owner->EmitterTime, Context.GetDistributionData(), InRandomStream);
		const float SpawnVelocity = Velocity.GetValue(Owner->EmitterTime, Context.GetDistributionData(), InRandomStream);
		const float LatheAngle = InRandomStream->FRand() * TwoPI;
		const FRotator DefaultDirectionRotater((int32)(SpawnAngle * ToRads * UUPerRad), (int32)(LatheAngle * UUPerRad), 0);
		const FRotationMatrix DefaultDirectionRotation(DefaultDirectionRotater);
		const FVector DefaultSpawnDirection = DefaultDirectionRotation.TransformVector(DefaultDirection);

		// Orientate the cone along the direction vector		
		FVector ForwardDirection = DefaultDirection;
		if (Direction != FVector::ZeroVector)
		{
			ForwardDirection = Direction.GetSafeNormal();
		}
		FVector UpDirection(0.0f, 0.0f, 1.0f);
		FVector RightDirection(1.0f, 0.0f, 0.0f);

		if ((ForwardDirection != UpDirection) && (-ForwardDirection != UpDirection))
		{
			RightDirection = UpDirection ^ ForwardDirection;
			UpDirection = ForwardDirection ^ RightDirection;
		}
		else
		{
			UpDirection = ForwardDirection ^ RightDirection;
			RightDirection = UpDirection ^ ForwardDirection;
		}

		FMatrix DirectionRotation;
		DirectionRotation.SetIdentity();
		DirectionRotation.SetAxis(0, RightDirection.GetSafeNormal());
		DirectionRotation.SetAxis(1, UpDirection.GetSafeNormal());
		DirectionRotation.SetAxis(2, ForwardDirection);
		FVector SpawnDirection = DirectionRotation.TransformVector(DefaultSpawnDirection);
	
		// Transform according to world and local space flags 
		if (!LODLevel->RequiredModule->bUseLocalSpace && !bInWorldSpace)
		{
			SpawnDirection = Context.GetTransform().TransformVector(SpawnDirection);
		}
		else if (LODLevel->RequiredModule->bUseLocalSpace && bInWorldSpace)
		{
			SpawnDirection = Context.GetTransform().InverseTransformVector(SpawnDirection);
		}

		// Set final velocity vector
		const FVector3f FinalVelocity(SpawnDirection * SpawnVelocity * OwnerScale);
		Particle.Velocity += FinalVelocity;
		Particle.BaseVelocity += FinalVelocity;
	}
}

void UParticleModuleVelocityCone::Render3DPreview(const FPreviewContext& Context)
{
#if WITH_EDITOR
	float ConeMaxAngle = 0.0f;
	float ConeMinAngle = 0.0f;
	Angle.GetOutRange(ConeMinAngle, ConeMaxAngle);

	float ConeMaxVelocity = 0.0f;
	float ConeMinVelocity = 0.0f;
	Velocity.GetOutRange(ConeMinVelocity, ConeMaxVelocity);

	float MaxLifetime = 0.0f;
	FParticleEmitterInstance* Owner = &Context.Owner;
	TArray<TObjectPtr<UParticleModule>>& Modules = Owner->SpriteTemplate->GetCurrentLODLevel(Owner)->Modules;
	for (int32 ModuleIndex = 0; ModuleIndex < Modules.Num(); ModuleIndex++)
	{
		UParticleModuleLifetimeBase* LifetimeMod = Cast<UParticleModuleLifetimeBase>(Modules[ModuleIndex]);
		if (LifetimeMod != NULL)
		{
			MaxLifetime = LifetimeMod->GetMaxLifetime();
			break;
		}
	}

	const int32 ConeSides = 16;
	const float ConeRadius = ConeMaxVelocity * MaxLifetime;

	// Calculate direction transform
	const FVector DefaultDirection(0.0f, 0.0f, 1.0f);
	const FVector ForwardDirection = (Direction != FVector::ZeroVector)? Direction.GetSafeNormal(): DefaultDirection;
	FVector UpDirection(0.0f, 0.0f, 1.0f);
	FVector RightDirection(1.0f, 0.0f, 0.0f);

	if ((ForwardDirection != UpDirection) && (-ForwardDirection != UpDirection))
	{
		RightDirection = UpDirection ^ ForwardDirection;
		UpDirection = ForwardDirection ^ RightDirection;
	}
	else
	{
		UpDirection = ForwardDirection ^ RightDirection;
		RightDirection = UpDirection ^ ForwardDirection;
	}

	FMatrix DirectionRotation;
	DirectionRotation.SetIdentity();
	DirectionRotation.SetAxis(0, RightDirection.GetSafeNormal());
	DirectionRotation.SetAxis(1, UpDirection.GetSafeNormal());
	DirectionRotation.SetAxis(2, ForwardDirection);

	// Calculate the owning actor's scale and rotation
	UParticleLODLevel* LODLevel	= Owner->SpriteTemplate->GetCurrentLODLevel(Owner);
	check(LODLevel);
	FVector OwnerScale(1.0f);
	FMatrix OwnerRotation(FMatrix::Identity);
	FVector LocalToWorldOrigin(0.0f);
	FMatrix LocalToWorld(FMatrix::Identity);
	if (UParticleSystemComponent* Component = Owner->Component.AsComponent())
	{
		if (AActor* Actor = Component->GetOwner())
		{
			if (bApplyOwnerScale == true)
			{
				OwnerScale = Context.GetTransform().GetScale3D();
			}

			OwnerRotation = FQuatRotationMatrix(Actor->GetActorQuat());
		}
	  LocalToWorldOrigin = Context.GetTransform().GetLocation();
	  LocalToWorld = Context.GetTransform().ToMatrixWithScale().RemoveTranslation();
	  LocalToWorld.RemoveScaling();
	}
	FMatrix Transform;
	Transform.SetIdentity();

	// DrawWireCone() draws a cone down the X axis, but this cone's default direction is down Z
	const FRotationMatrix XToZRotation(FRotator((int32)(UE_HALF_PI * 10430), 0, 0));
	Transform *= XToZRotation;

	// Apply scale
	Transform.SetAxis(0, Transform.GetScaledAxis( EAxis::X ) * OwnerScale.X);
	Transform.SetAxis(1, Transform.GetScaledAxis( EAxis::Y ) * OwnerScale.Y);
	Transform.SetAxis(2, Transform.GetScaledAxis( EAxis::Z ) * OwnerScale.Z);

	// Apply direction transform
	Transform *= DirectionRotation;

	// Transform according to world and local space flags 
	if (!LODLevel->RequiredModule->bUseLocalSpace && !bInWorldSpace)
	{
		Transform *= LocalToWorld;
	}
	else if (LODLevel->RequiredModule->bUseLocalSpace && bInWorldSpace)
	{
		Transform *= OwnerRotation;
		Transform *= LocalToWorld.InverseFast();
	}
	else if (!bInWorldSpace)
	{
		Transform *= OwnerRotation;
	}

	// Apply translation
	Transform.SetOrigin(LocalToWorldOrigin);

	TArray<FVector> OuterVerts;
	TArray<FVector> InnerVerts;

	FPrimitiveDrawInterface* PDI = Context.PDI;

	// Draw inner and outer cones
	DrawWireCone(PDI, InnerVerts, Transform, ConeRadius, ConeMinAngle, ConeSides, ModuleEditorColor, SDPG_World);
	DrawWireCone(PDI, OuterVerts, Transform, ConeRadius, ConeMaxAngle, ConeSides, ModuleEditorColor, SDPG_World);

	// Draw radial spokes
	for (int32 i = 0; i < ConeSides; ++i)
	{
		PDI->DrawLine( OuterVerts[i], InnerVerts[i], ModuleEditorColor, SDPG_World );
	}
#endif
}
