// Copyright Epic Games, Inc. All Rights Reserved.

#include "Particles/ParticlePerfStats.h"
#include "Engine/World.h"
#include "Particles/ParticlePerfStatsManager.h"

CSV_DEFINE_CATEGORY_MODULE(ENGINE_API, Particles, false);

#if WITH_PARTICLE_PERF_STATS

#include "CanvasTypes.h"
#include "Engine/Engine.h"
#include "Engine/Font.h"
#include "HAL/FileManager.h"
#include "Particles/ParticleSystem.h"
#include "UObject/UObjectIterator.h"
#include "Particles/ParticleSystem.h"
#include "Misc/CoreDelegates.h"
#include "Misc/OutputDeviceArchiveWrapper.h"
#include "Misc/Paths.h"
#include "Misc/Paths.h"
#include "CanvasTypes.h"
#include "Engine/Font.h"
#include "Particles/ParticleSystemComponent.h"
#include "Serialization/ArchiveCountMem.h"

DECLARE_STATS_GROUP(TEXT("ParticleStats"), STATGROUP_ParticleStats, STATCAT_Advanced);
DECLARE_CYCLE_STAT(TEXT("Particle Stats Tick [GT]"), STAT_ParticleStats_TickGT, STATGROUP_ParticleStats);
DECLARE_CYCLE_STAT(TEXT("Particle Stats Tick [RT]"), STAT_ParticleStats_TickRT, STATGROUP_ParticleStats);

TAtomic<bool> FParticlePerfStats::bCSVStatsEnabled(false);
TAtomic<bool> FParticlePerfStats::bStatsEnabled(true);
TAtomic<int32> FParticlePerfStats::WorldStatsReaders(0);
TAtomic<int32> FParticlePerfStats::SystemStatsReaders(0);
TAtomic<int32> FParticlePerfStats::ComponentStatsReaders(0);

FDelegateHandle FParticlePerfStatsManager::BeginFrameHandle;
#if CSV_PROFILER_STATS
FDelegateHandle FParticlePerfStatsManager::CSVStartHandle;
FDelegateHandle FParticlePerfStatsManager::CSVEndHandle;
#endif

FCriticalSection FParticlePerfStatsManager::WorldToPerfStatsGuard;
TMap<TWeakObjectPtr<const UWorld>, TUniquePtr<FParticlePerfStats>> FParticlePerfStatsManager::WorldToPerfStats;
TArray<TUniquePtr<FParticlePerfStats>> FParticlePerfStatsManager::FreeWorldStatsPool;

#if WITH_PER_SYSTEM_PARTICLE_PERF_STATS
FCriticalSection FParticlePerfStatsManager::SystemToPerfStatsGuard;
TMap<TWeakObjectPtr<const UFXSystemAsset>, TUniquePtr<FParticlePerfStats>> FParticlePerfStatsManager::SystemToPerfStats;
TArray<TUniquePtr<FParticlePerfStats>> FParticlePerfStatsManager::FreeSystemStatsPool;
#endif

#if WITH_PER_COMPONENT_PARTICLE_PERF_STATS
FCriticalSection FParticlePerfStatsManager::ComponentToPerfStatsGuard;
TMap<TWeakObjectPtr<const UFXSystemComponent>, TUniquePtr<FParticlePerfStats>> FParticlePerfStatsManager::ComponentToPerfStats;
TArray<TUniquePtr<FParticlePerfStats>> FParticlePerfStatsManager::FreeComponentStatsPool;
#endif

TArray<FParticlePerfStatsListenerPtr, TInlineAllocator<8>> FParticlePerfStatsManager::Listeners;

#if ENABLE_PARTICLE_PERF_STATS_RENDER
TMap<TWeakObjectPtr<UWorld>, TSharedPtr<FParticlePerfStatsListener_DebugRender, ESPMode::ThreadSafe>> FParticlePerfStatsManager::DebugRenderListenerUsers;
#endif


bool GbLocalStatsEnabled = FParticlePerfStats::bStatsEnabled;
static FAutoConsoleVariableRef CVarParticlePerfStatsEnabled(
	TEXT("fx.ParticlePerfStats.Enabled"),
	GbLocalStatsEnabled,
	TEXT("Used to control if stat gathering is enabled or not.\n"),
	FConsoleVariableDelegate::CreateLambda([](IConsoleVariable* CVar)
		{
			check(CVar);
			FParticlePerfStats::SetStatsEnabled(CVar->GetBool());
		}),
	ECVF_Default
);

int32 GParticlePerfStatsMemoryMode = 1;
static FAutoConsoleVariableRef CVarFXDetailedCSVMemoryStats(
	TEXT("fx.ParticlePerfStats.MemoryMode"),
	GParticlePerfStatsMemoryMode,
	TEXT("Gathers approximate memory information depending on the mode.\n")
	TEXT("0 = Disabled.\n")
	TEXT("1 = Component Approximate Size (default - low perf impact).\n")
	TEXT("2 = Asset Size + Component Approximate Size (low perf impact).\n")
	TEXT("3 = Asset Size + Component Resource Size (high performance impact)."),
	ECVF_Default
);

static FAutoConsoleCommandWithWorldAndArgs GParticlePerfStatsRunTest(
	TEXT("fx.ParticlePerfStats.RunTest"),
	TEXT("Runs for a number of frames then logs out the results.\nArg0 = NumFrames.\nArg1 = Gather World Stats (default 0).\nArg2 = Gather System Stats (default 1).\nArg3 = Gather Component Stats (default 0)."),
	FConsoleCommandWithWorldAndArgsDelegate::CreateLambda(
		[](const TArray<FString>& Args, UWorld* World)
		{
			if (Args.Num() < 1)
			{
				return;
			}

			const int32 NumFrames = FCString::Atoi(*Args[0]);
			if (NumFrames <= 0)
			{
				return;
			}

			bool bGatherWorldStats = false;
			bool bGatherSystemStats = true;
			bool bGatherComponentStats = false;

			if (Args.Num() > 1)
			{
				bGatherWorldStats = FCString::Atoi(*Args[1]) != 0;
			}
			if (Args.Num() > 2)
			{
				bGatherSystemStats = FCString::Atoi(*Args[2]) != 0;
			}
			if (Args.Num() > 3)
			{
				bGatherComponentStats = FCString::Atoi(*Args[3]) != 0;
			}

			FParticlePerfStatsListenerPtr NewTimedTest = MakeShared<FParticlePerfStatsListener_TimedTest, ESPMode::ThreadSafe>(NumFrames, bGatherWorldStats, bGatherSystemStats, bGatherComponentStats);
			FParticlePerfStatsManager::AddListener(NewTimedTest);
		}
	)
);

void FParticlePerfStatsManager::AddListener(FParticlePerfStatsListenerPtr Listener, bool bReset)
{
	if (bReset)
	{
		Reset();
	}

	if (FParticlePerfStats::GetStatsEnabled())
	{
		Listeners.Add(Listener);
		Listener->Begin();

		//Ensure we're gathering stats.
		if (Listener->NeedsWorldStats())
		{
			FParticlePerfStats::AddWorldStatReader();
		}
		if (Listener->NeedsSystemStats() || Listener->NeedsMemoryStats())
		{
			FParticlePerfStats::AddSystemStatReader();
		}
		if (Listener->NeedsComponentStats())
		{
			FParticlePerfStats::AddComponentStatReader();
		}
	}
}

void FParticlePerfStatsManager::RemoveListener(FParticlePerfStatsListener* Listener)
{
	RemoveListener(Listener->AsShared());
}

void FParticlePerfStatsManager::RemoveListener(FParticlePerfStatsListenerPtr Listener)
{
	//Pass a ptr off to the RT just so we can ensure it's lifetime past any RT commands it may have issued.
	ENQUEUE_RENDER_COMMAND(FRemoveParticlePerfStatsListenerCmd)
	(
		[Listener](FRHICommandListImmediate& RHICmdList)mutable
		{
			Listener.Reset();
		}
	);

	Listener->End();
	Listeners.Remove(Listener);

	//Unregister the listener from the stats so we will stop gathering if there are no listeners.
	if (Listener->NeedsWorldStats())
	{
		FParticlePerfStats::RemoveWorldStatReader();
	}
	if (Listener->NeedsSystemStats() || Listener->NeedsMemoryStats())
	{
		FParticlePerfStats::RemoveSystemStatReader();
	}
	if (Listener->NeedsComponentStats())
	{
		FParticlePerfStats::RemoveComponentStatReader();
	}
}

void FParticlePerfStatsManager::Reset()
{
	FlushRenderingCommands();

	{
		FScopeLock ScopeLock(&FParticlePerfStatsManager::WorldToPerfStatsGuard);
		for (auto& WorldStatsPair : WorldToPerfStats)
		{
			if(const UWorld* World = WorldStatsPair.Key.GetEvenIfUnreachable())
			{
				World->ParticlePerfStats = nullptr;
			}
		}
		WorldToPerfStats.Empty();
	}

	#if WITH_PER_SYSTEM_PARTICLE_PERF_STATS
	{
		FScopeLock ScopeLock(&FParticlePerfStatsManager::SystemToPerfStatsGuard);
		for (auto& SystemStatsPair : SystemToPerfStats)
		{
			if (const UFXSystemAsset* System = SystemStatsPair.Key.GetEvenIfUnreachable())
			{
				System->ParticlePerfStats = nullptr;
			}
		}
		SystemToPerfStats.Empty();
	}
	#endif

#if WITH_PER_COMPONENT_PARTICLE_PERF_STATS
	{
		FScopeLock ScopeLock(&FParticlePerfStatsManager::ComponentToPerfStatsGuard);
		for (TObjectIterator<UFXSystemComponent> CompIt; CompIt; ++CompIt)
		{
			CompIt->ParticlePerfStats = nullptr;
			CompIt->MarkRenderStateDirty();
		}
		ComponentToPerfStats.Empty();
	}
#else
	{
		for (TObjectIterator<UFXSystemComponent> CompIt; CompIt; ++CompIt)
		{
			CompIt->MarkRenderStateDirty();
		}
	}
#endif
}

void FParticlePerfStatsManager::Tick()
{
	if (FParticlePerfStats::ShouldGatherStats())
	{
		SCOPE_CYCLE_COUNTER(STAT_ParticleStats_TickGT);

		check(Listeners.Num() > 0);

#if WITH_PER_SYSTEM_PARTICLE_PERF_STATS
		// If anyone needs memory stats we tick those now
		if (GParticlePerfStatsMemoryMode != 0 && Listeners.ContainsByPredicate([](const FParticlePerfStatsListenerPtr& Listener) { return Listener->NeedsMemoryStats();}))
		{
			checkSlow(FParticlePerfStats::GetGatherSystemStats() && FParticlePerfStats::GetStatsEnabled());
			const bool bIncludeAssetSize = GParticlePerfStatsMemoryMode > 1;
			const bool bComponentUseResourceSize = GParticlePerfStatsMemoryMode == 3;

			// Gather per loaded asset cost
			if (bIncludeAssetSize)
			{
				for (TObjectIterator<UFXSystemAsset> It; It; ++It)
				{
					UFXSystemAsset* FXAsset = *It;
					if (!IsValid(FXAsset) || FXAsset->IsUnreachable() || FXAsset->HasAnyFlags(EObjectFlags::RF_ClassDefaultObject))
					{
						continue;
					}

					FParticlePerfStats* Stats = FParticlePerfStatsManager::GetSystemPerfStats(FXAsset);
					if (!Stats->MemoryKB_Asset.IsSet())
					{
						FResourceSizeEx ResSize = FResourceSizeEx(EResourceSizeMode::EstimatedTotal);
						FXAsset->GetResourceSizeEx(ResSize);
						Stats->MemoryKB_Asset = ResSize.GetTotalMemoryBytes();
					}
				}
			}

			// Gather per component cost
			for (TObjectIterator<UFXSystemComponent> It; It; ++It)
			{
				UFXSystemComponent* FXComponent = *It;
				UFXSystemAsset* FXAsset = IsValid(FXComponent) ? FXComponent->GetFXSystemAsset() : nullptr;
				if (!IsValid(FXAsset) || FXComponent->IsUnreachable() || FXComponent->HasAnyFlags(EObjectFlags::RF_ClassDefaultObject))
				{
					continue;
				}

				FParticlePerfStats* Stats = FParticlePerfStatsManager::GetSystemPerfStats(FXAsset);
				if (bComponentUseResourceSize)
				{
					Stats->GetGameThreadStats().MemoryBytes += FArchiveCountMem(FXComponent).GetMax();

					FResourceSizeEx CompResSize = FResourceSizeEx(EResourceSizeMode::EstimatedTotal);
					FXComponent->GetResourceSizeEx(CompResSize);
					Stats->GetGameThreadStats().MemoryBytes += CompResSize.GetTotalMemoryBytes();
				}
				else
				{
					Stats->GetGameThreadStats().MemoryBytes += FXComponent->GetApproxMemoryUsage();
				}
			}
		}
#endif

		//Tick our listeners so they can consume the finished frame data.
		TArray<FParticlePerfStatsListenerPtr, TInlineAllocator<8>> ToRemove;
		for (FParticlePerfStatsListenerPtr& Listener : Listeners)
		{
			if (Listener.IsValid() == false || (Listener.IsUnique() && Listener->AllowOrphaned() == false) || Listener->Tick() == false)
			{
				ToRemove.Add(Listener);
			}
		}

		//Kick off the RT tick for listeners and stats
		ENQUEUE_RENDER_COMMAND(FParticlePerfStatsListenersRTTick)
		(
			[ListenersRT=TArray<FParticlePerfStatsListenerPtr, TInlineAllocator<8>>(Listeners)](FRHICommandListImmediate& RHICmdList)
			{
				SCOPE_CYCLE_COUNTER(STAT_ParticleStats_TickRT);
				for (FParticlePerfStatsListenerPtr Listener : ListenersRT)
				{
					Listener->TickRT();
				}

				//Reset current frame data
				{
					FScopeLock ScopeLock(&FParticlePerfStatsManager::WorldToPerfStatsGuard);
					for (auto it = WorldToPerfStats.CreateIterator(); it; ++it)
					{
						it.Value()->TickRT();
					}
				}
#if WITH_PER_SYSTEM_PARTICLE_PERF_STATS
				{
					FScopeLock ScopeLock(&FParticlePerfStatsManager::SystemToPerfStatsGuard);
					for (auto it = SystemToPerfStats.CreateIterator(); it; ++it)
					{
						it.Value()->TickRT();
					}
				} 
#endif

#if WITH_PER_COMPONENT_PARTICLE_PERF_STATS
				{
					FScopeLock ScopeLock(&FParticlePerfStatsManager::ComponentToPerfStatsGuard);
					for (auto it = ComponentToPerfStats.CreateIterator(); it; ++it)
					{
						it.Value()->TickRT();
					}
				}
#endif
			}
		);

		//Reset current frame data
		{
			FScopeLock ScopeLock(&FParticlePerfStatsManager::WorldToPerfStatsGuard);
			for (auto it = WorldToPerfStats.CreateIterator(); it; ++it)
			{
				if(it->Key.Get())
				{
					it.Value()->Tick();
				}
				else
				{
					FreeWorldStatsPool.Emplace(it.Value().Release());
					for (FParticlePerfStatsListenerPtr& Listener : Listeners)
					{
						Listener->OnRemoveWorld(it.Key());
					}
					it.RemoveCurrent();
				}				
			}
		}
#if WITH_PER_SYSTEM_PARTICLE_PERF_STATS
		{
			FScopeLock ScopeLock(&FParticlePerfStatsManager::SystemToPerfStatsGuard);
			for (auto it = SystemToPerfStats.CreateIterator(); it; ++it)
			{
				if (it->Key.Get())
				{
					it.Value()->Tick();
				}
				else
				{
#if WITH_PARTICLE_PERF_CSV_STATS
					it.Value()->ResetStatNames();
#endif
					it.Value()->MemoryKB_Asset.Reset();	// Note not clear above as this path can be taken without CSV stats
					FreeSystemStatsPool.Emplace(it.Value().Release());
					for (FParticlePerfStatsListenerPtr& Listener : Listeners)
					{
						Listener->OnRemoveSystem(it.Key());
					}
					it.RemoveCurrent();
				}
			}
		}
#endif

#if WITH_PER_COMPONENT_PARTICLE_PERF_STATS
		{
			FScopeLock ScopeLock(&FParticlePerfStatsManager::ComponentToPerfStatsGuard);
			for (auto it = ComponentToPerfStats.CreateIterator(); it; ++it)
			{
				if (it->Key.Get())
				{
					it.Value()->Tick();
				}
				else
				{
					FreeComponentStatsPool.Emplace(it.Value().Release());
					for (FParticlePerfStatsListenerPtr& Listener : Listeners)
					{
						Listener->OnRemoveComponent(it.Key());
					}
					it.RemoveCurrent();
				}
			}
		}
#endif
		//Remove any listeners that are done.
		for (FParticlePerfStatsListenerPtr& Listener : ToRemove)
		{
			RemoveListener(Listener);
		}
	}
	else
	{
		//Ensure any existing listeners are removed if stats have been disabled.
		while (Listeners.Num())
		{
			RemoveListener(Listeners.Last());
		}
	}
}

#if WITH_PARTICLE_PERF_CSV_STATS
void FParticlePerfStats::PopulateStatNames(const FName InName)
{
	if (CSVStat_Total == NAME_None)
	{
		const FString StatName = InName.ToString();
		CSVStat_Total = FName(*FString::Printf(TEXT("Total/%s"), *StatName), 0, false);
		CSVStat_GTOnly = FName(*FString::Printf(TEXT("GTOnly/%s"), *StatName), 0, false);
		CSVStat_InstAvgGT = FName(*FString::Printf(TEXT("InstAvgGT/%s"), *StatName), 0, false);
		CSVStat_RT = FName(*FString::Printf(TEXT("RT/%s"), *StatName), 0, false);
		CSVStat_InstAvgRT = FName(*FString::Printf(TEXT("InstAvgRT/%s"), *StatName), 0, false);
		CSVStat_GPU = FName(*FString::Printf(TEXT("GPU/%s"), *StatName), 0, false);
		CSVStat_InstAvgGPU = FName(*FString::Printf(TEXT("InstAvgGPU/%s"), *StatName), 0, false);
		CSVStat_Count = FName(*FString::Printf(TEXT("Count/%s"), *StatName), 0, false);
		CSVStat_Activation = FName(*FString::Printf(TEXT("Activation/%s"), *StatName), 0, false);
		CSVStat_Waits = FName(*FString::Printf(TEXT("Waits/%s"), *StatName), 0, false);
		CSVStat_Culled = FName(*FString::Printf(TEXT("Culled/%s"), *StatName), 0, false);
		CSVStat_MemoryKB = FName(*FString::Printf(TEXT("MemoryKB/%s"), *StatName), 0, false);
	}
}

void FParticlePerfStats::ResetStatNames()
{
	CSVStat_Total = NAME_None;
	CSVStat_GTOnly = NAME_None;
	CSVStat_InstAvgGT = NAME_None;
	CSVStat_RT = NAME_None;
	CSVStat_InstAvgRT = NAME_None;
	CSVStat_GPU = NAME_None;
	CSVStat_InstAvgGPU = NAME_None;
	CSVStat_Count = NAME_None;
	CSVStat_Activation = NAME_None;
	CSVStat_Waits = NAME_None;
	CSVStat_Culled = NAME_None;
	CSVStat_MemoryKB = NAME_None;
}
#endif

FParticlePerfStats* FParticlePerfStatsManager::GetWorldPerfStats(const UWorld* World)
{
	checkSlow(World && FParticlePerfStats::GetGatherWorldStats() && FParticlePerfStats::GetStatsEnabled());

	if (World->ParticlePerfStats == nullptr)
	{
		FScopeLock ScopeLock(&WorldToPerfStatsGuard);
		TUniquePtr<FParticlePerfStats>& PerfStats = WorldToPerfStats.FindOrAdd(World);
		if (PerfStats == nullptr)
		{
			if (FreeWorldStatsPool.Num())
			{
				PerfStats = FreeWorldStatsPool.Pop();
			}
			else
			{
				PerfStats.Reset(new FParticlePerfStats());
			}
		}
		World->ParticlePerfStats = PerfStats.Get();

		for (auto& Listener : Listeners)
		{
			Listener->OnAddWorld(World);
		}
	}
	return World->ParticlePerfStats;
}

FParticlePerfStats* FParticlePerfStatsManager::GetSystemPerfStats(const UFXSystemAsset* FXAsset)
{
#if WITH_PER_SYSTEM_PARTICLE_PERF_STATS
	checkSlow(FXAsset && FParticlePerfStats::GetGatherSystemStats() && FParticlePerfStats::GetStatsEnabled());
	if (FXAsset->ParticlePerfStats == nullptr)
	{
		FScopeLock ScopeLock(&SystemToPerfStatsGuard);
		TUniquePtr<FParticlePerfStats>& PerfStats = SystemToPerfStats.FindOrAdd(FXAsset);
		if (PerfStats == nullptr)
		{
			if (FreeSystemStatsPool.Num())
			{
				PerfStats = FreeSystemStatsPool.Pop();
			}
			else
			{
				PerfStats.Reset(new FParticlePerfStats());
			}
		}

		FXAsset->ParticlePerfStats = PerfStats.Get();
#if WITH_PARTICLE_PERF_CSV_STATS
		FXAsset->ParticlePerfStats->PopulateStatNames(FXAsset->GetFName());
#endif

		for (auto& Listener : Listeners)
		{
			Listener->OnAddSystem(FXAsset);
		}
	}
	return FXAsset->ParticlePerfStats;
#else
	return nullptr;
#endif
}

FParticlePerfStats* FParticlePerfStatsManager::GetComponentPerfStats(const UFXSystemComponent* FXComponent)
{
#if WITH_PER_COMPONENT_PARTICLE_PERF_STATS
	checkSlow(FXComponent && FParticlePerfStats::GetGatherComponentStats() && FParticlePerfStats::GetStatsEnabled());
	if (FXComponent->ParticlePerfStats == nullptr)
	{
		FScopeLock ScopeLock(&ComponentToPerfStatsGuard);
		TUniquePtr<FParticlePerfStats>& PerfStats = ComponentToPerfStats.FindOrAdd(FXComponent);
		if (PerfStats == nullptr)
		{
			if (FreeComponentStatsPool.Num())
			{
				PerfStats = FreeComponentStatsPool.Pop();
			}
			else
			{
				PerfStats.Reset(new FParticlePerfStats());
			}

		}
		FXComponent->ParticlePerfStats = PerfStats.Get();

		for (auto& Listener : Listeners)
		{
			Listener->OnAddComponent(FXComponent);
		}
	}
	return FXComponent->ParticlePerfStats;
#else
	return nullptr;
#endif
}

void FParticlePerfStatsManager::TogglePerfStatsRender(UWorld* World)
{
#if ENABLE_PARTICLE_PERF_STATS_RENDER
	if (auto* Found = DebugRenderListenerUsers.Find(World))
	{
		//Already have an entry so we're toggling rendering off. Remove.
		RemoveListener(*Found);
		DebugRenderListenerUsers.Remove(World);
	}
	else
	{
		//Need not found. Add a new listener for this world.
		TSharedPtr<FParticlePerfStatsListener_DebugRender, ESPMode::ThreadSafe> NewListener = MakeShared<FParticlePerfStatsListener_DebugRender, ESPMode::ThreadSafe>();
		DebugRenderListenerUsers.Add(World) = NewListener;
		AddListener(NewListener);
	}
#endif
}

int32 FParticlePerfStatsManager::RenderStats(class UWorld* World, class FViewport* Viewport, class FCanvas* Canvas, int32 X, int32 Y, const FVector* ViewLocation, const FRotator* ViewRotation)
{
#if ENABLE_PARTICLE_PERF_STATS_RENDER
	//We shouldn't get into this rendering function unless we have registered users.
	if (auto* DebugRenderListener = DebugRenderListenerUsers.Find(World))
	{
		return (*DebugRenderListener)->RenderStats(World, Viewport, Canvas, X, Y, ViewLocation, ViewRotation);
	}
#endif 
	return Y;
}

void FParticlePerfStatsManager::OnStartup()
{
	BeginFrameHandle = FCoreDelegates::OnBeginFrame.AddStatic(Tick);
#if CSV_PROFILER_STATS && !UE_BUILD_SHIPPING
	if (FCsvProfiler* CSVProfiler = FCsvProfiler::Get())
	{
		CSVStartHandle = CSVProfiler->OnCSVProfileStart().AddStatic(FParticlePerfStatsListener_CSVProfiler::OnCSVStart);
		CSVEndHandle = CSVProfiler->OnCSVProfileEnd().AddStatic(FParticlePerfStatsListener_CSVProfiler::OnCSVEnd);
	}
#endif
}

void FParticlePerfStatsManager::OnShutdown()
{
	FCoreDelegates::OnBeginFrame.Remove(BeginFrameHandle);
#if CSV_PROFILER_STATS && !UE_BUILD_SHIPPING
	if (FCsvProfiler* CSVProfiler = FCsvProfiler::Get())
	{
		CSVProfiler->OnCSVProfileStart().Remove(CSVStartHandle);
		CSVProfiler->OnCSVProfileEnd().Remove(CSVEndHandle);
	}
#endif
}

//////////////////////////////////////////////////////////////////////////

FParticlePerfStats* FParticlePerfStats::GetWorldPerfStats(const UWorld* World)
{
	return FParticlePerfStatsManager::GetWorldPerfStats(World);
}

FParticlePerfStats* FParticlePerfStats::GetSystemPerfStats(const UFXSystemAsset* FXAsset)
{
	return FParticlePerfStatsManager::GetSystemPerfStats(FXAsset);
}

FParticlePerfStats* FParticlePerfStats::GetComponentPerfStats(const UFXSystemComponent* FXComponent)
{
	return FParticlePerfStatsManager::GetComponentPerfStats(FXComponent);
}

void FParticlePerfStats::ResetGT()
{
	check(IsInGameThread());
	GetGameThreadStats().Reset();
}

void FParticlePerfStats::ResetRT()
{
	check(IsInActualRenderingThread());
	GetRenderThreadStats().Reset();
	GetGPUStats().Reset();
}

FParticlePerfStats::FParticlePerfStats()
{
}

void FParticlePerfStats::Reset(bool bSyncWithRT)
{
	check(IsInGameThread());

	ResetGT();

	if (bSyncWithRT)
	{
		FlushRenderingCommands();
		ResetRT();
	}
	else
	{
		ENQUEUE_RENDER_COMMAND(FResetParticlePerfStats)
		(
			[&](FRHICommandListImmediate& RHICmdList)
			{
				ResetRT();
			}
		);
	}
}

void FParticlePerfStats::Tick()
{
	check(IsInGameThread());
	GetGameThreadStats().Reset();
}

void FParticlePerfStats::TickRT()
{	
	check(IsInRenderingThread());
	GetRenderThreadStats().Reset();
	GetGPUStats().Reset();
}

//////////////////////////////////////////////////////////////////////////

FAccumulatedParticlePerfStats_GT::FAccumulatedParticlePerfStats_GT()
{
	Reset();
}

void FAccumulatedParticlePerfStats_GT::Reset()
{
	FAccumulatedParticlePerfStats::ResetMaxArray(MaxPerFrameTotalCycles);
	FAccumulatedParticlePerfStats::ResetMaxArray(MaxPerInstanceCycles);

	NumFrames = 0;
	AccumulatedStats.Reset();
}

void FAccumulatedParticlePerfStats_GT::Tick(FParticlePerfStats& Stats)
{
	FParticlePerfStats_GT& GTStats = Stats.GetGameThreadStats();
	if (GTStats.NumInstances > 0)
	{
		++NumFrames;
		AccumulatedStats.NumInstances += GTStats.NumInstances;
		AccumulatedStats.TickGameThreadCycles += GTStats.TickGameThreadCycles;
		AccumulatedStats.TickConcurrentCycles += GTStats.TickConcurrentCycles;
		AccumulatedStats.FinalizeCycles += GTStats.FinalizeCycles;
		AccumulatedStats.EndOfFrameCycles += GTStats.EndOfFrameCycles;

		FAccumulatedParticlePerfStats::AddMax(MaxPerFrameTotalCycles, GTStats.GetTotalCycles());
		FAccumulatedParticlePerfStats::AddMax(MaxPerInstanceCycles, GTStats.GetPerInstanceAvgCycles());
	}
}

//////////////////////////////////////////////////////////////////////////

FAccumulatedParticlePerfStats_RT::FAccumulatedParticlePerfStats_RT()
{
	Reset();
}

void FAccumulatedParticlePerfStats_RT::Reset()
{
	FAccumulatedParticlePerfStats::ResetMaxArray(MaxPerFrameTotalCycles);
	FAccumulatedParticlePerfStats::ResetMaxArray(MaxPerInstanceCycles);

	NumFrames = 0;
	AccumulatedStats.Reset();
}

void FAccumulatedParticlePerfStats_RT::Tick(FParticlePerfStats& Stats)
{
	FParticlePerfStats_RT& RTStats = Stats.GetRenderThreadStats();
	if (RTStats.NumInstances > 0)
	{
		++NumFrames;
		AccumulatedStats.NumInstances += RTStats.NumInstances;
		AccumulatedStats.RenderUpdateCycles += RTStats.RenderUpdateCycles;
		AccumulatedStats.GetDynamicMeshElementsCycles += RTStats.GetDynamicMeshElementsCycles;

		FAccumulatedParticlePerfStats::AddMax(MaxPerFrameTotalCycles, RTStats.GetTotalCycles());
		FAccumulatedParticlePerfStats::AddMax(MaxPerInstanceCycles, RTStats.GetPerInstanceAvgCycles());
	}
}

//////////////////////////////////////////////////////////////////////////

FAccumulatedParticlePerfStats_GPU::FAccumulatedParticlePerfStats_GPU()
{
	Reset();
}

void FAccumulatedParticlePerfStats_GPU::Reset()
{
	FAccumulatedParticlePerfStats::ResetMaxArray(MaxPerFrameTotalMicroseconds);
	FAccumulatedParticlePerfStats::ResetMaxArray(MaxPerInstanceMicroseconds);

	NumFrames = 0;
	AccumulatedStats.Reset();
}

void FAccumulatedParticlePerfStats_GPU::Tick(FParticlePerfStats& Stats)
{
	FParticlePerfStats_GPU& GPUStats = Stats.GetGPUStats();
	if (GPUStats.NumInstances > 0)
	{
		++NumFrames;
		AccumulatedStats.NumInstances += GPUStats.NumInstances;
		AccumulatedStats.TotalMicroseconds += GPUStats.TotalMicroseconds;

		FAccumulatedParticlePerfStats::AddMax(MaxPerFrameTotalMicroseconds, GPUStats.GetTotalMicroseconds());
		FAccumulatedParticlePerfStats::AddMax(MaxPerInstanceMicroseconds, GPUStats.GetPerInstanceAvgMicroseconds());
	}
}

//////////////////////////////////////////////////////////////////////////

FAccumulatedParticlePerfStats::FAccumulatedParticlePerfStats()
{
	ResetGT();
	ResetRT();
}

void FAccumulatedParticlePerfStats::ResetGT()
{
	GameThreadStats.Reset();
}

void FAccumulatedParticlePerfStats::ResetRT()
{
	RenderThreadStats.Reset();
	GPUStats.Reset();
}

void FAccumulatedParticlePerfStats::Reset(bool bSyncWithRT)
{
	ResetGT();

	if (bSyncWithRT)
	{
		FlushRenderingCommands();
		ResetRT();
	}
	else
	{
		//Not syncing with RT so must update these on the RT.
		ENQUEUE_RENDER_COMMAND(FResetAccumulatedParticlePerfMaxRT)
		(
			[&](FRHICommandListImmediate& RHICmdList)
			{
				ResetRT();
			}
		);
	}
}

void FAccumulatedParticlePerfStats::Tick(FParticlePerfStats& Stats)
{
	check(IsInGameThread());
	GameThreadStats.Tick(Stats);
}

void FAccumulatedParticlePerfStats::TickRT(FParticlePerfStats& Stats)
{
	check(IsInRenderingThread());
	RenderThreadStats.Tick(Stats);
	GPUStats.Tick(Stats);
}

void FAccumulatedParticlePerfStats::AddMax(TArray<uint64, TInlineAllocator<ACCUMULATED_PARTICLE_PERF_STAT_MAX_SAMPLES>>& MaxArray, int64 NewValue)
{
	int32 InsertIndex;
	InsertIndex = MaxArray.IndexOfByPredicate([&](uint32 v) {return NewValue > v; });
	if (InsertIndex != INDEX_NONE)
	{
		MaxArray.Pop(EAllowShrinking::No);
		MaxArray.Insert(NewValue, InsertIndex);
	}
};

void FAccumulatedParticlePerfStats::ResetMaxArray(TArray<uint64, TInlineAllocator<ACCUMULATED_PARTICLE_PERF_STAT_MAX_SAMPLES>>& MaxArray)
{
	MaxArray.SetNumUninitialized(ACCUMULATED_PARTICLE_PERF_STAT_MAX_SAMPLES);
	for (int32 i = 0; i < ACCUMULATED_PARTICLE_PERF_STAT_MAX_SAMPLES; ++i)
	{
		MaxArray[i] = 0;
	}
};

//////////////////////////////////////////////////////////////////////////

void FParticlePerfStatsListener_GatherAll::Begin()
{
	//Init our map of accumulated stats.
	FScopeLock Lock(&AccumulatedStatsGuard);

	FParticlePerfStatsManager::ForAllWorldStats(
		[&](TWeakObjectPtr<const UWorld>& WeakWorld, TUniquePtr<FParticlePerfStats>& Stats)
		{
			AccumulatedWorldStats.Add(WeakWorld) = MakeUnique<FAccumulatedParticlePerfStats>();
		}
	);

#if WITH_PER_SYSTEM_PARTICLE_PERF_STATS
	FParticlePerfStatsManager::ForAllSystemStats(
		[&](TWeakObjectPtr<const UFXSystemAsset>& WeakSystem, TUniquePtr<FParticlePerfStats>& Stats)
		{
			AccumulatedSystemStats.Add(WeakSystem) = MakeUnique<FAccumulatedParticlePerfStats>();
		}
	);
#endif

#if WITH_PER_COMPONENT_PARTICLE_PERF_STATS
	FParticlePerfStatsManager::ForAllComponentStats(
		[&](TWeakObjectPtr<const UFXSystemComponent>& WeakComponent, TUniquePtr<FParticlePerfStats>& Stats)
		{
			AccumulatedComponentStats.Add(WeakComponent) = MakeUnique<FAccumulatedParticlePerfStats>();
		}
	);
#endif
}

void FParticlePerfStatsListener_GatherAll::End()
{
	FScopeLock Lock(&AccumulatedStatsGuard);
	AccumulatedWorldStats.Empty();
#if WITH_PER_SYSTEM_PARTICLE_PERF_STATS
	AccumulatedSystemStats.Empty();
#endif
#if WITH_PER_COMPONENT_PARTICLE_PERF_STATS
	AccumulatedComponentStats.Empty();
#endif
}

template<typename T, typename TFunc>
void FParticlePerfStatsListener_GatherAll::TickStats_Internal(TMap<TWeakObjectPtr<T>, TUniquePtr<FAccumulatedParticlePerfStats>>& StatsMap, TFunc Func)
{
	TArray<TWeakObjectPtr<T>, TInlineAllocator<8>> ToRemove;
	for (TPair<TWeakObjectPtr<T>, TUniquePtr<FAccumulatedParticlePerfStats>>& Pair : StatsMap)
	{
		if (T* Key = Pair.Key.Get())
		{
			FAccumulatedParticlePerfStats& Stats = *Pair.Value;

			if (FParticlePerfStats* CurrentFrameStats = Key->ParticlePerfStats)
			{
				Func(Stats, *CurrentFrameStats);
			}
		}
		else
		{
			ToRemove.Add(Pair.Key);
		}
	}
	for (TWeakObjectPtr<T>& Item : ToRemove)
	{
		StatsMap.Remove(Item);
	}
}

bool FParticlePerfStatsListener_GatherAll::Tick()
{
	auto TickFunc = [](FAccumulatedParticlePerfStats& Stats, FParticlePerfStats& FrameStats)
	{
		Stats.Tick(FrameStats);
	};

	FScopeLock Lock(&AccumulatedStatsGuard);
	TickStats_Internal(AccumulatedWorldStats, TickFunc);
#if WITH_PER_SYSTEM_PARTICLE_PERF_STATS
	TickStats_Internal(AccumulatedSystemStats, TickFunc);
#endif
#if WITH_PER_COMPONENT_PARTICLE_PERF_STATS
	TickStats_Internal(AccumulatedComponentStats, TickFunc);
#endif

	return true;
}

void FParticlePerfStatsListener_GatherAll::TickRT()
{
	auto TickFunc = [](FAccumulatedParticlePerfStats& Stats, FParticlePerfStats& FrameStats)
	{
		Stats.TickRT(FrameStats);
	};

	FScopeLock Lock(&AccumulatedStatsGuard);
	TickStats_Internal(AccumulatedWorldStats, TickFunc);
#if WITH_PER_SYSTEM_PARTICLE_PERF_STATS
	TickStats_Internal(AccumulatedSystemStats, TickFunc);
#endif
#if WITH_PER_COMPONENT_PARTICLE_PERF_STATS
	TickStats_Internal(AccumulatedComponentStats, TickFunc);
#endif
}

void FParticlePerfStatsListener_GatherAll::OnAddWorld(const TWeakObjectPtr<const UWorld>& NewWorld)
{
	if(bGatherWorldStats)
	{
		FScopeLock Lock(&AccumulatedStatsGuard);
		AccumulatedWorldStats.Add(NewWorld) = MakeUnique<FAccumulatedParticlePerfStats>();
	}
}

void FParticlePerfStatsListener_GatherAll::OnRemoveWorld(const TWeakObjectPtr<const UWorld>& World)
{
	if (bGatherWorldStats)
	{
		FScopeLock Lock(&AccumulatedStatsGuard);
		AccumulatedWorldStats.Remove(World);
	}
}

#if WITH_PER_SYSTEM_PARTICLE_PERF_STATS
void FParticlePerfStatsListener_GatherAll::OnAddSystem(const TWeakObjectPtr<const UFXSystemAsset>& NewSystem)
{
	if (bGatherSystemStats)
	{
		FScopeLock Lock(&AccumulatedStatsGuard);
		AccumulatedSystemStats.Add(NewSystem) = MakeUnique<FAccumulatedParticlePerfStats>();
	}
}

void FParticlePerfStatsListener_GatherAll::OnRemoveSystem(const TWeakObjectPtr<const UFXSystemAsset>& System)
{
	if (bGatherSystemStats)
	{
		FScopeLock Lock(&AccumulatedStatsGuard);
		AccumulatedSystemStats.Remove(System);
	}
}
#endif

#if WITH_PER_COMPONENT_PARTICLE_PERF_STATS
void FParticlePerfStatsListener_GatherAll ::OnAddComponent(const TWeakObjectPtr<const UFXSystemComponent>& NewComponent)
{
	if (bGatherComponentStats)
	{
		FScopeLock Lock(&AccumulatedStatsGuard);
		AccumulatedComponentStats.Add(NewComponent) = MakeUnique<FAccumulatedParticlePerfStats>();
	}
}

void FParticlePerfStatsListener_GatherAll::OnRemoveComponent(const TWeakObjectPtr<const UFXSystemComponent>& Component)
{
	if (bGatherComponentStats)
	{
		FScopeLock Lock(&AccumulatedStatsGuard);
		AccumulatedComponentStats.Remove(Component);
	}
}
#endif

void FParticlePerfStatsListener_GatherAll::DumpStatsToDevice(FOutputDevice& Ar)
{
	FlushRenderingCommands();

	FString tempString;

	Ar.Logf(TEXT(",**** Particle Performance Stats"));
	Ar.Logf(TEXT(",Name,Average PerFrame GameThread,Average PerInstance GameThread,Average PerFrame RenderThread,Average PerInstance RenderThread,NumFrames,Total Instances,Total Tick GameThread,Total Tick Concurrent,Total Finalize,Total End Of Frame,Total Render Update,Total Get Dynamic Mesh Elements,Max PerFrame GameThread,Max Range PerFrame GameThread,Max PerFrame RenderThread,Max Range PerFrame RenderThread"));

	auto WriteStats = [&tempString](FOutputDevice& Ar, const FString& Name, FAccumulatedParticlePerfStats* PerfStats)
	{
		if(PerfStats == nullptr)
		{
			return;
		}

		const FAccumulatedParticlePerfStats_GT& GTStats = PerfStats->GetGameThreadStats();
		const FAccumulatedParticlePerfStats_RT& RTStats = PerfStats->GetRenderThreadStats();

		if ((GTStats.NumFrames == 0 && RTStats.NumFrames == 0) || (GTStats.AccumulatedStats.NumInstances == 0 && RTStats.AccumulatedStats.NumInstances == 0))
		{
			return;
		}

		const uint64 TotalGameThread = GTStats.GetTotalCycles();
		const uint64 TotalRenderThread = RTStats.GetTotalCycles();

		const uint64 MaxPerFrameTotalGameThreadFirst = GTStats.MaxPerFrameTotalCycles.Num() > 0 ? GTStats.MaxPerFrameTotalCycles[0] : 0;
		const uint64 MaxPerFrameTotalGameThreadLast = GTStats.MaxPerFrameTotalCycles.Num() > 0 ? GTStats.MaxPerFrameTotalCycles.Last() : 0;

		const uint64 MaxPerFrameTotalRenderThreadFirst = RTStats.MaxPerFrameTotalCycles.Num() > 0 ? RTStats.MaxPerFrameTotalCycles[0] : 0;
		const uint64 MaxPerFrameTotalRenderThreadLast = RTStats.MaxPerFrameTotalCycles.Num() > 0 ? RTStats.MaxPerFrameTotalCycles.Last() : 0;
		//TODO: Add per instance max?

		tempString.Reset();
		tempString.Appendf(TEXT(",%s"), *Name);
		tempString.Appendf(TEXT(",%u"), uint32(FPlatformTime::ToMilliseconds64(GTStats.GetPerFrameAvgCycles()) * 1000.0));
		tempString.Appendf(TEXT(",%u"), uint32(FPlatformTime::ToMilliseconds64(GTStats.GetPerInstanceAvgCycles()) * 1000.0));
		tempString.Appendf(TEXT(",%u"), uint32(FPlatformTime::ToMilliseconds64(RTStats.GetPerFrameAvgCycles()) * 1000.0));
		tempString.Appendf(TEXT(",%u"), uint32(FPlatformTime::ToMilliseconds64(RTStats.GetPerInstanceAvgCycles()) * 1000.0));
		tempString.Appendf(TEXT(",%u"), uint32(GTStats.NumFrames));
		tempString.Appendf(TEXT(",%u"), uint32(GTStats.AccumulatedStats.NumInstances));
		tempString.Appendf(TEXT(",%u"), uint32(FPlatformTime::ToMilliseconds64(GTStats.AccumulatedStats.TickGameThreadCycles) * 1000.0));
		tempString.Appendf(TEXT(",%u"), uint32(FPlatformTime::ToMilliseconds64(GTStats.AccumulatedStats.TickConcurrentCycles) * 1000.0));
		tempString.Appendf(TEXT(",%u"), uint32(FPlatformTime::ToMilliseconds64(GTStats.AccumulatedStats.FinalizeCycles) * 1000.0));
		tempString.Appendf(TEXT(",%u"), uint32(FPlatformTime::ToMilliseconds64(GTStats.AccumulatedStats.EndOfFrameCycles) * 1000.0));
		tempString.Appendf(TEXT(",%u"), uint32(FPlatformTime::ToMilliseconds64(RTStats.AccumulatedStats.RenderUpdateCycles) * 1000.0));
		tempString.Appendf(TEXT(",%u"), uint32(FPlatformTime::ToMilliseconds64(RTStats.AccumulatedStats.GetDynamicMeshElementsCycles) * 1000.0));

		tempString.Appendf(TEXT(",%u,[ "), uint32(FPlatformTime::ToMilliseconds64(GTStats.MaxPerFrameTotalCycles.Num() > 0 ? GTStats.MaxPerFrameTotalCycles[0] : 0) * 1000.0));
		for (uint64 v : GTStats.MaxPerFrameTotalCycles)
		{
			tempString.Appendf(TEXT("%u "), uint32(FPlatformTime::ToMilliseconds64(v) * 1000.0));
		}
		tempString.Append(TEXT("]"));

		tempString.Appendf(TEXT(",%u,[ "), uint32(FPlatformTime::ToMilliseconds64(RTStats.MaxPerInstanceCycles.Num() > 0 ? RTStats.MaxPerInstanceCycles[0] : 0) * 1000.0));
		for (uint64 v : RTStats.MaxPerFrameTotalCycles)
		{
			tempString.Appendf(TEXT("%u "), uint32(FPlatformTime::ToMilliseconds64(v) * 1000.0));
		}
		tempString.Append(TEXT("]"));

		Ar.Log(*tempString);
	};

	if(bGatherWorldStats)
	{
		Ar.Logf(TEXT(",** Per World Stats"));
		for (auto it = AccumulatedWorldStats.CreateIterator(); it; ++it)
		{
			if (auto* Key = it.Key().Get())
			{
				WriteStats(Ar, Key->GetFName().ToString(), it.Value().Get());
			}
		}
	}

#if WITH_PER_SYSTEM_PARTICLE_PERF_STATS
	if (bGatherSystemStats)
	{
		Ar.Logf(TEXT(",** Per System Stats"));
		for (auto it = AccumulatedSystemStats.CreateIterator(); it; ++it)
		{
			if (auto* Key = it.Key().Get())
			{
				WriteStats(Ar, Key->GetFName().ToString(), it.Value().Get());
			}
		}
	}
#endif

#if WITH_PER_COMPONENT_PARTICLE_PERF_STATS
	if (bGatherComponentStats)
	{
		Ar.Logf(TEXT(",** Per Component Stats"));
		for (auto it = AccumulatedComponentStats.CreateIterator(); it; ++it)
		{
			if (auto* Key = it.Key().Get())
			{
				WriteStats(Ar, Key->GetFName().ToString(), it.Value().Get());
			}
		}
	}
#endif
}

void FParticlePerfStatsListener_GatherAll::DumpStatsToFile()
{
#if !UE_BUILD_SHIPPING
	const FString PathName = FPaths::ProfilingDir() + TEXT("ParticlePerf");
	IFileManager::Get().MakeDirectory(*PathName);

	const FString Filename = FString::Printf(TEXT("ParticlePerf-%s.csv"), *FDateTime::Now().ToString(TEXT("%d-%H.%M.%S")));
	const FString FilePath = PathName / Filename;

	if (FArchive* FileAr = IFileManager::Get().CreateDebugFileWriter(*FilePath))
	{
		TUniquePtr<FOutputDeviceArchiveWrapper> FileArWrapper(new FOutputDeviceArchiveWrapper(FileAr));
		DumpStatsToDevice(*FileArWrapper.Get());
		delete FileAr;
	}
#endif
}


#if WITH_PARTICLE_PERF_STATS
FAccumulatedParticlePerfStats* FParticlePerfStatsListener_GatherAll::GetStats(const UWorld* World)
{
	if (TUniquePtr<FAccumulatedParticlePerfStats>* StatsPtr = AccumulatedWorldStats.Find(World))
	{
		return StatsPtr->Get();
	}
	return nullptr;
}
#endif
#if WITH_PER_SYSTEM_PARTICLE_PERF_STATS
FAccumulatedParticlePerfStats* FParticlePerfStatsListener_GatherAll::GetStats(const UFXSystemAsset* System)
{
	if (TUniquePtr<FAccumulatedParticlePerfStats>* StatsPtr = AccumulatedSystemStats.Find(System))
	{
		return StatsPtr->Get();
	}
	return nullptr;
}
#endif
#if WITH_PER_COMPONENT_PARTICLE_PERF_STATS
FAccumulatedParticlePerfStats* FParticlePerfStatsListener_GatherAll::GetStats(const UFXSystemComponent* Component) 
{
	if (TUniquePtr<FAccumulatedParticlePerfStats>* StatsPtr = AccumulatedComponentStats.Find(Component))
	{
		return StatsPtr->Get();
	}
	return nullptr;
}
#endif

//////////////////////////////////////////////////////////////////////////

FParticlePerfStatsListener_TimedTest::FParticlePerfStatsListener_TimedTest(int32 NumFrames, bool bInGatherWorldStats, bool bInGatherSystemStats, bool bInGatherComponentStats)
	: FParticlePerfStatsListener_GatherAll(bInGatherWorldStats, bInGatherSystemStats, bInGatherComponentStats)
	, FramesRemaining(NumFrames)
{

}

void FParticlePerfStatsListener_TimedTest::End()
{
	//TODO: Move this stuff into the listeners themselves with some utilities in the manager so each listener can customize it's output more.
	if (GLog != nullptr)
	{
		DumpStatsToDevice(*GLog);
	}
	DumpStatsToFile();
}

bool FParticlePerfStatsListener_TimedTest::Tick()
{
	FParticlePerfStatsListener_GatherAll::Tick();

	return --FramesRemaining > 0;
}

//////////////////////////////////////////////////////////////////////////

#if WITH_PARTICLE_PERF_CSV_STATS

void OnDetailedCSVStatsEnabledChanged(IConsoleVariable* Variable);

static FAutoConsoleVariable CVarWriteDetailedCSVStats(
	TEXT("fx.DetailedCSVStats"),
	false,
	TEXT("If true, we write detailed partilce stats to the CSV profiler."),
	FConsoleVariableDelegate::CreateStatic(&OnDetailedCSVStatsEnabledChanged),
	ECVF_Default | ECVF_RenderThreadSafe
);

void OnDetailedCSVStatsEnabledChanged(IConsoleVariable* Variable)
{
	const bool bDetailedCSVStats = CVarWriteDetailedCSVStats->GetBool();

	FParticlePerfStats::SetCSVStatsEnabled(bDetailedCSVStats);
	
	FCsvProfiler* CSVProfiler = FCsvProfiler::Get();
	if (CSVProfiler)
	{
		CSVProfiler->EnableCategoryByIndex(CSV_CATEGORY_INDEX(Particles), bDetailedCSVStats);
		CSV_METADATA(TEXT("DetailedFXStats"), bDetailedCSVStats ? TEXT("1") : TEXT("0"));
	}

	if (bDetailedCSVStats)
	{
		if (CSVProfiler && CSVProfiler->IsCapturing())
		{			
			FParticlePerfStatsListener_CSVProfiler::OnCSVStart();
		}
	}
	else
	{
		FParticlePerfStatsListener_CSVProfiler::OnCSVEnd();
	}
}

FParticlePerfStatsListenerPtr FParticlePerfStatsListener_CSVProfiler::CSVListener;
void FParticlePerfStatsListener_CSVProfiler::OnCSVStart()
{
	const bool bDetailedCSVStats = CVarWriteDetailedCSVStats->GetBool();
	if (bDetailedCSVStats && CSVListener.IsValid() == false)
	{
		CSVListener = MakeShared<FParticlePerfStatsListener_CSVProfiler, ESPMode::ThreadSafe>();
		FParticlePerfStatsManager::AddListener(CSVListener);
	}
}

void FParticlePerfStatsListener_CSVProfiler::OnCSVEnd()
{
	if (CSVListener.IsValid())
	{
		FParticlePerfStatsManager::RemoveListener(CSVListener.Get());
		CSVListener.Reset();
	}
}

bool FParticlePerfStatsListener_CSVProfiler::Tick()
{
	check(FParticlePerfStats::GetCSVStatsEnabled());

	FParticlePerfStatsListener_GatherAll::Tick();

	if (FCsvProfiler* CSVProfiler = FCsvProfiler::Get())
	{
#if WITH_PER_SYSTEM_PARTICLE_PERF_STATS
		FParticlePerfStatsManager::ForAllSystemStats(
			[&](TWeakObjectPtr<const UFXSystemAsset>& WeakSystem, TUniquePtr<FParticlePerfStats>& Stats)
			{
				if (Stats->GetGameThreadStats().NumInstances > 0)
				{
					if (const UFXSystemAsset* System = WeakSystem.Get())
					{
						const float TotalTime = FPlatformTime::ToMilliseconds64(Stats->GetGameThreadStats().GetTotalCycles()) * 1000.0f;
						const float GTTime = FPlatformTime::ToMilliseconds64(Stats->GetGameThreadStats().GetTotalCycles_GTOnly()) * 1000.0f;
						const float AvgTime = FPlatformTime::ToMilliseconds64(Stats->GetGameThreadStats().GetPerInstanceAvgCycles()) * 1000.0f;
						const int32 Count = (int32)Stats->GetGameThreadStats().NumInstances;
						const float Activation = FPlatformTime::ToMilliseconds64(Stats->GetGameThreadStats().ActivationCycles) * 1000.0f;
						const float Wait = FPlatformTime::ToMilliseconds64(Stats->GetGameThreadStats().WaitCycles) * 1000.0f;

						Stats->PopulateStatNames(System->GetFName());

						CSVProfiler->RecordCustomStat(Stats->CSVStat_Total, CSV_CATEGORY_INDEX(Particles), TotalTime, ECsvCustomStatOp::Set);
						CSVProfiler->RecordCustomStat(Stats->CSVStat_GTOnly, CSV_CATEGORY_INDEX(Particles), GTTime, ECsvCustomStatOp::Set);
						CSVProfiler->RecordCustomStat(Stats->CSVStat_InstAvgGT, CSV_CATEGORY_INDEX(Particles), AvgTime, ECsvCustomStatOp::Set);
						CSVProfiler->RecordCustomStat(Stats->CSVStat_Count, CSV_CATEGORY_INDEX(Particles), Count, ECsvCustomStatOp::Set);

						CSVProfiler->RecordCustomStat(Stats->CSVStat_Activation, CSV_CATEGORY_INDEX(Particles), Activation, ECsvCustomStatOp::Set);
						CSVProfiler->RecordCustomStat(Stats->CSVStat_Waits, CSV_CATEGORY_INDEX(Particles), Wait, ECsvCustomStatOp::Set);

						if (!Stats->CSVStat_MemoryKB.IsNone())
						{
							const uint64 MemoryBytes = Stats->GetGameThreadStats().MemoryBytes + Stats->MemoryKB_Asset.Get(0);
							const int32 MemoryKB = (int32)FMath::DivideAndRoundUp(MemoryBytes, 1024ull);
							CSVProfiler->RecordCustomStat(Stats->CSVStat_MemoryKB, CSV_CATEGORY_INDEX(Particles), MemoryKB, ECsvCustomStatOp::Set);
						}
					}
				}
			}
		);
#endif
		return true;
	}

	return false;
}

void FParticlePerfStatsListener_CSVProfiler::TickRT()
{
	FParticlePerfStatsListener_GatherAll::TickRT();
	if (FCsvProfiler* CSVProfiler = FCsvProfiler::Get())
	{
#if WITH_PER_SYSTEM_PARTICLE_PERF_STATS
		FParticlePerfStatsManager::ForAllSystemStats(
			[&](TWeakObjectPtr<const UFXSystemAsset>& WeakSystem, TUniquePtr<FParticlePerfStats>& Stats)
			{
				if (const UFXSystemAsset* System = WeakSystem.Get())
				{
					Stats->PopulateStatNames(System->GetFName());

					const float RTTime = FPlatformTime::ToMilliseconds64(Stats->GetRenderThreadStats().GetTotalCycles()) * 1000.0f;
					const float RTAvgTime = FPlatformTime::ToMilliseconds64(Stats->GetRenderThreadStats().GetPerInstanceAvgCycles()) * 1000.0f;
					CSVProfiler->RecordCustomStat(Stats->CSVStat_RT, CSV_CATEGORY_INDEX(Particles), RTTime, ECsvCustomStatOp::Set);
					CSVProfiler->RecordCustomStat(Stats->CSVStat_InstAvgRT, CSV_CATEGORY_INDEX(Particles), RTAvgTime, ECsvCustomStatOp::Set);

					const float GpuTime = float(Stats->GetGPUStats().GetTotalMicroseconds());
					const float GpuAvgTime = float(Stats->GetGPUStats().GetPerInstanceAvgMicroseconds());
					CSVProfiler->RecordCustomStat(Stats->CSVStat_GPU, CSV_CATEGORY_INDEX(Particles), GpuTime, ECsvCustomStatOp::Set);
					CSVProfiler->RecordCustomStat(Stats->CSVStat_InstAvgGPU, CSV_CATEGORY_INDEX(Particles), GpuAvgTime, ECsvCustomStatOp::Set);
				}
			}
		);
#endif
	}
}

void FParticlePerfStatsListener_CSVProfiler::End()
{
	if (GLog != nullptr)
	{
		DumpStatsToDevice(*GLog);
	}
	DumpStatsToFile();
}

#endif//WITH_PARTICLE_PERF_CSV_STATS

//////////////////////////////////////////////////////////////////////////

int32 FParticlePerfStatsListener_DebugRender::RenderStats(class UWorld* World, class FViewport* Viewport, class FCanvas* Canvas, int32 /*X*/, int32 Y, const FVector* ViewLocation, const FRotator* ViewRotation)
{
#if WITH_PER_SYSTEM_PARTICLE_PERF_STATS
	UFont* Font = GEngine->GetSmallFont();
	check(Font != nullptr);

	float CharWidth = 0.0f;
	float CharHeight = 0.0f;
	Font->GetCharSize('W', CharWidth, CharHeight);
	const float ColumnWidth = 32 * CharWidth;
	const int32 FontHeight = Font->GetMaxCharHeight() + 2.0f;

	int32 X = 100;

	// Draw background
	{
		int32 NumRows = 0;
		for (auto it = AccumulatedSystemStats.CreateIterator(); it; ++it)
		{
			FAccumulatedParticlePerfStats* PerfStats = it.Value().Get();
			if (PerfStats == nullptr || PerfStats->GetGameThreadStats().NumFrames == 0)
			{
				continue;
			}
			++NumRows;
		}
	}

	static FLinearColor HeaderBackground = FLinearColor(1.0f, 1.0f, 1.0f, 0.5f);
	static FLinearColor BackgroundColors[] = { FLinearColor(0.6f, 0.6f, 0.6f, 0.5f), FLinearColor(0.4f, 0.4f, 0.4f, 0.5f) };

	// Display Header
	Canvas->DrawTile(X - 2, Y - 1, (ColumnWidth * 5) + 4, FontHeight, 0.0f, 0.0f, 1.0f, 1.0f, HeaderBackground);
	Canvas->DrawShadowedString(X + ColumnWidth * 0, Y, TEXT("System Name"), Font, FLinearColor::Yellow);
	Canvas->DrawShadowedString(X + ColumnWidth * 1, Y, TEXT("Average PerFrame GT | GT CNC | RT | GPU"), Font, FLinearColor::Yellow);
	Canvas->DrawShadowedString(X + ColumnWidth * 2, Y, TEXT("Average PerInstance GT | GT CNC | RT | GPU"), Font, FLinearColor::Yellow);
	Canvas->DrawShadowedString(X + ColumnWidth * 3, Y, TEXT("Peak PerFrame GT | RT"), Font, FLinearColor::Yellow);
	Y += FontHeight;

	FString tempString;
	int32 RowNum = 0;
	for (auto it = AccumulatedSystemStats.CreateIterator(); it; ++it)
	{
		FAccumulatedParticlePerfStats* PerfStats = it.Value().Get();
		if (PerfStats == nullptr)
		{
			continue;
		}

		const FAccumulatedParticlePerfStats_GT& GTStats = PerfStats->GetGameThreadStats();
		const FAccumulatedParticlePerfStats_RT& RTStats = PerfStats->GetRenderThreadStats();
		const FAccumulatedParticlePerfStats_GPU& GPUStats = PerfStats->GetGPUStats();
		const UFXSystemAsset* System = it.Key().Get();
		if (GTStats.NumFrames == 0 || RTStats.NumFrames == 0 || System == nullptr)
		{
			continue;
		}

		// Background
		++RowNum;
		Canvas->DrawTile(X - 2, Y - 1, (ColumnWidth * 5) + 4, FontHeight, 0.0f, 0.0f, 1.0f, 1.0f, BackgroundColors[RowNum & 1]);

		// System Name
		FString SystemName = System->GetFName().ToString();
		Canvas->DrawShadowedString(X + ColumnWidth * 0, Y, *SystemName, Font, FLinearColor::Yellow);

		// Average Per Frame
		tempString.Reset();
		tempString.Appendf(
			TEXT("%4u | %4u | %4u | %4u"),
			uint32(FPlatformTime::ToMilliseconds64(GTStats.AccumulatedStats.TickGameThreadCycles + GTStats.AccumulatedStats.FinalizeCycles) * 1000.0 / double(GTStats.NumFrames)),
			uint32(FPlatformTime::ToMilliseconds64(GTStats.AccumulatedStats.TickConcurrentCycles + GTStats.AccumulatedStats.EndOfFrameCycles) * 1000.0 / double(GTStats.NumFrames)),
			uint32(FPlatformTime::ToMilliseconds64(RTStats.AccumulatedStats.RenderUpdateCycles + RTStats.AccumulatedStats.GetDynamicMeshElementsCycles) * 1000.0 / double(RTStats.NumFrames)),
			uint32((double(GPUStats.AccumulatedStats.TotalMicroseconds) / 1000.0) / double(GTStats.NumFrames))
		);
		Canvas->DrawShadowedString(X + ColumnWidth * 1, Y, *tempString, Font, FLinearColor::Yellow);

		// Average Per Instances
		tempString.Reset();
		tempString.Appendf(
			TEXT("%4u | %4u | %4u | %4u"),
			uint32(FPlatformTime::ToMilliseconds64(GTStats.AccumulatedStats.TickGameThreadCycles + GTStats.AccumulatedStats.FinalizeCycles) * 1000.0 / double(GTStats.AccumulatedStats.NumInstances)),
			uint32(FPlatformTime::ToMilliseconds64(GTStats.AccumulatedStats.TickConcurrentCycles + GTStats.AccumulatedStats.EndOfFrameCycles) * 1000.0 / double(GTStats.AccumulatedStats.NumInstances)),
			uint32(FPlatformTime::ToMilliseconds64(RTStats.AccumulatedStats.RenderUpdateCycles + RTStats.AccumulatedStats.GetDynamicMeshElementsCycles) * 1000.0 / double(RTStats.AccumulatedStats.NumInstances)),
			uint32((double(GPUStats.AccumulatedStats.TotalMicroseconds) / 1000.0) / double(GTStats.AccumulatedStats.NumInstances))
		);
		Canvas->DrawShadowedString(X + ColumnWidth * 2, Y, *tempString, Font, FLinearColor::Yellow);

		// Peak Per Frame
		tempString.Reset();
		tempString.Append(TEXT("GT[ "));
		for (uint64 v : GTStats.MaxPerFrameTotalCycles)
		{
			tempString.Appendf(TEXT("%4u "), uint32(FPlatformTime::ToMilliseconds64(v) * 1000.0));
		}
		tempString.Append(TEXT("] RT["));
		for (uint64 v : RTStats.MaxPerFrameTotalCycles)
		{
			tempString.Appendf(TEXT("%4u "), uint32(FPlatformTime::ToMilliseconds64(v) * 1000.0));
		}
		tempString.Append(TEXT("]"));
		Canvas->DrawShadowedString(X + ColumnWidth * 3, Y, *tempString, Font, FLinearColor::Yellow);

		Y += FontHeight;
	}
#endif
	return Y;
}

//////////////////////////////////////////////////////////////////////////

#endif //WITH_PARTICLE_PERF_STATS
