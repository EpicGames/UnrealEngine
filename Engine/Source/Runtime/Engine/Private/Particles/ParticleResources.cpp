// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	ParticleResources.cpp: Implementation of global particle resources.
=============================================================================*/

#include "ParticleResources.h"
#include "Containers/ClosableMpscQueue.h"
#include "RHI.h"
#include "RHICommandList.h"
#include "Experimental/Containers/HazardPointer.h"

/** The size of the scratch vertex buffer. */
const int32 GParticleScratchVertexBufferSize = 64 * (1 << 10); // 64KB

/**
 * Creates a vertex buffer holding texture coordinates for the four corners of a sprite.
 */
void FParticleTexCoordVertexBuffer::InitRHI(FRHICommandListBase& RHICmdList)
{
	const FRHIBufferCreateDesc CreateDesc =
		FRHIBufferCreateDesc::CreateVertex<FVector2f>(TEXT("FParticleTexCoordVertexBuffer"), 4 * MAX_PARTICLES_PER_INSTANCE)
		.AddUsage(EBufferUsageFlags::Static)
		.SetInitialState(ERHIAccess::VertexOrIndexBuffer)
		.SetInitActionInitializer();

	TRHIBufferInitializer<FVector2f> Vertices = RHICmdList.CreateBufferInitializer(CreateDesc);

	for (uint32 SpriteIndex = 0; SpriteIndex < MAX_PARTICLES_PER_INSTANCE; ++SpriteIndex)
	{
		Vertices[SpriteIndex*4 + 0] = FVector2f(0.0f, 0.0f);
		Vertices[SpriteIndex*4 + 1] = FVector2f(0.0f, 1.0f);
		Vertices[SpriteIndex*4 + 2] = FVector2f(1.0f, 1.0f);
		Vertices[SpriteIndex*4 + 3] = FVector2f(1.0f, 0.0f);
	}

	VertexBufferRHI = Vertices.Finalize();
}

/** Global particle texture coordinate vertex buffer. */
TGlobalResource<FParticleTexCoordVertexBuffer> GParticleTexCoordVertexBuffer;

/**
 * Creates a vertex buffer holding texture coordinates for eight corners of a polygon.
 */
void FParticleEightTexCoordVertexBuffer::InitRHI(FRHICommandListBase& RHICmdList)
{
	const FRHIBufferCreateDesc CreateDesc =
		FRHIBufferCreateDesc::CreateVertex<FVector2f>(TEXT("FParticleEightTexCoordVertexBuffer"), 8 * MAX_PARTICLES_PER_INSTANCE)
		.AddUsage(EBufferUsageFlags::Static)
		.SetInitialState(ERHIAccess::VertexOrIndexBuffer)
		.SetInitActionInitializer();

	TRHIBufferInitializer<FVector2f> Vertices = RHICmdList.CreateBufferInitializer(CreateDesc);

	for (uint32 SpriteIndex = 0; SpriteIndex < MAX_PARTICLES_PER_INSTANCE; ++SpriteIndex)
	{
		// The contents of this buffer does not matter, whenever it is used, cutout geometry will override
		Vertices[SpriteIndex*8 + 0] = FVector2f(0.0f, 0.0f);
		Vertices[SpriteIndex*8 + 1] = FVector2f(0.0f, 1.0f);
		Vertices[SpriteIndex*8 + 2] = FVector2f(1.0f, 1.0f);
		Vertices[SpriteIndex*8 + 3] = FVector2f(1.0f, 0.0f);
		Vertices[SpriteIndex*8 + 4] = FVector2f(1.0f, 0.0f);
		Vertices[SpriteIndex*8 + 5] = FVector2f(1.0f, 0.0f);
		Vertices[SpriteIndex*8 + 6] = FVector2f(1.0f, 0.0f);
		Vertices[SpriteIndex*8 + 7] = FVector2f(1.0f, 0.0f);
	}

	VertexBufferRHI = Vertices.Finalize();
}

/** Global particle texture coordinate vertex buffer. */
TGlobalResource<FParticleEightTexCoordVertexBuffer> GParticleEightTexCoordVertexBuffer;

/**
 * Creates an index buffer for drawing an individual sprite.
 */
void FParticleIndexBuffer::InitRHI(FRHICommandListBase& RHICmdList)
{
	// Instanced path needs only MAX_PARTICLES_PER_INSTANCE,
	// but using the maximum needed for the non-instanced path
	// in prep for future flipping of both instanced and non-instanced at runtime.
	const uint32 MaxParticles = 65536 / 4;

	const FRHIBufferCreateDesc CreateDesc =
		FRHIBufferCreateDesc::CreateIndex<uint16>(TEXT("FParticleIndexBuffer"), 6 * MaxParticles)
		.AddUsage(EBufferUsageFlags::Static)
		.SetInitialState(ERHIAccess::VertexOrIndexBuffer)
		.SetInitActionInitializer();

	TRHIBufferInitializer<uint16> Indices = RHICmdList.CreateBufferInitializer(CreateDesc);

	for (uint32 SpriteIndex = 0; SpriteIndex < MaxParticles; ++SpriteIndex)
	{
		Indices[SpriteIndex*6 + 0] = SpriteIndex*4 + 0;
		Indices[SpriteIndex*6 + 1] = SpriteIndex*4 + 2;
		Indices[SpriteIndex*6 + 2] = SpriteIndex*4 + 3;
		Indices[SpriteIndex*6 + 3] = SpriteIndex*4 + 0;
		Indices[SpriteIndex*6 + 4] = SpriteIndex*4 + 1;
		Indices[SpriteIndex*6 + 5] = SpriteIndex*4 + 2;
	}

	IndexBufferRHI = Indices.Finalize();
}

/** Global particle index buffer. */
TGlobalResource<FParticleIndexBuffer> GParticleIndexBuffer;

/**
 * Creates an index buffer for drawing an individual sprite.
 */
void FSixTriangleParticleIndexBuffer::InitRHI(FRHICommandListBase& RHICmdList)
{
	// Instanced path needs only MAX_PARTICLES_PER_INSTANCE,
	// but using the maximum needed for the non-instanced path
	// in prep for future flipping of both instanced and non-instanced at runtime.
	const uint32 MaxParticles = 65536 / 8;

	const FRHIBufferCreateDesc CreateDesc =
		FRHIBufferCreateDesc::CreateIndex<uint16>(TEXT("FSixTriangleParticleIndexBuffer"), 6 * 3 * MaxParticles)
		.AddUsage(EBufferUsageFlags::Static)
		.SetInitialState(ERHIAccess::VertexOrIndexBuffer)
		.SetInitActionInitializer();

	TRHIBufferInitializer<uint16> Indices = RHICmdList.CreateBufferInitializer(CreateDesc);

	for (uint32 SpriteIndex = 0; SpriteIndex < MaxParticles; ++SpriteIndex)
	{
		Indices[SpriteIndex*18 + 0] = SpriteIndex*8 + 0;
		Indices[SpriteIndex*18 + 1] = SpriteIndex*8 + 1;
		Indices[SpriteIndex*18 + 2] = SpriteIndex*8 + 2;
		Indices[SpriteIndex*18 + 3] = SpriteIndex*8 + 0;
		Indices[SpriteIndex*18 + 4] = SpriteIndex*8 + 2;
		Indices[SpriteIndex*18 + 5] = SpriteIndex*8 + 3;

		Indices[SpriteIndex*18 + 6] = SpriteIndex*8 + 0;
		Indices[SpriteIndex*18 + 7] = SpriteIndex*8 + 3;
		Indices[SpriteIndex*18 + 8] = SpriteIndex*8 + 4;
		Indices[SpriteIndex*18 + 9] = SpriteIndex*8 + 0;
		Indices[SpriteIndex*18 + 10] = SpriteIndex*8 + 4;
		Indices[SpriteIndex*18 + 11] = SpriteIndex*8 + 5;

		Indices[SpriteIndex*18 + 12] = SpriteIndex*8 + 0;
		Indices[SpriteIndex*18 + 13] = SpriteIndex*8 + 5;
		Indices[SpriteIndex*18 + 14] = SpriteIndex*8 + 6;
		Indices[SpriteIndex*18 + 15] = SpriteIndex*8 + 0;
		Indices[SpriteIndex*18 + 16] = SpriteIndex*8 + 6;
		Indices[SpriteIndex*18 + 17] = SpriteIndex*8 + 7;
	}

	IndexBufferRHI = Indices.Finalize();
}

/** Global particle index buffer. */
TGlobalResource<FSixTriangleParticleIndexBuffer> GSixTriangleParticleIndexBuffer;

/**
 * Creates a scratch vertex buffer available for dynamic draw calls.
 */
void FParticleScratchVertexBuffer::InitRHI(FRHICommandListBase& RHICmdList)
{
	// Create a scratch vertex buffer for injecting particles and rendering tiles.
	const FRHIBufferCreateDesc CreateDesc =
		FRHIBufferCreateDesc::CreateVertex(TEXT("FParticleScratchVertexBuffer"), GParticleScratchVertexBufferSize)
		.AddUsage(EBufferUsageFlags::Volatile | EBufferUsageFlags::ShaderResource)
		.DetermineInitialState();

	VertexBufferRHI = RHICmdList.CreateBuffer(CreateDesc);
	VertexBufferSRV_A32B32G32R32F = RHICmdList.CreateShaderResourceView(
		VertexBufferRHI, 
		FRHIViewDesc::CreateBufferSRV()
			.SetType(FRHIViewDesc::EBufferType::Typed)
			.SetFormat(PF_A32B32G32R32F));
}

FParticleShaderParamRef FParticleScratchVertexBuffer::GetShaderParam()
{
	return VertexBufferSRV_A32B32G32R32F;
}

FParticleBufferParamRef FParticleScratchVertexBuffer::GetBufferParam()
{
	return VertexBufferRHI;
}

/** Release RHI resources. */
void FParticleScratchVertexBuffer::ReleaseRHI()
{
	VertexBufferSRV_A32B32G32R32F.SafeRelease();
	FVertexBuffer::ReleaseRHI();
}

/** The global scratch vertex buffer. */
TGlobalResource<FParticleScratchVertexBuffer> GParticleScratchVertexBuffer;
