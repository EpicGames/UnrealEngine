// Copyright Epic Games, Inc. All Rights Reserved.

/*==============================================================================
	ParticleSimulationGPU.h: Interface to GPU particle simulation.
==============================================================================*/

#pragma once

#include "CoreMinimal.h"

/*------------------------------------------------------------------------------
	Constants to tune memory and performance for GPU particle simulation.
------------------------------------------------------------------------------*/

/** The texture size allocated for GPU simulation. */
extern int32 GParticleSimulationTextureSizeX;
extern int32 GParticleSimulationTextureSizeY;

/** The tile size. Texture space is allocated in TileSize x TileSize units. */
extern const int32 GParticleSimulationTileSize;
extern const int32 GParticlesPerTile;

