// Copyright Epic Games, Inc. All Rights Reserved.

/*==============================================================================
	ParticleSortingGPU.cpp: Sorting GPU particles.
==============================================================================*/

#include "Particles/ParticleSortingGPU.h"
#include "Particles/ParticleSimulationGPU.h"
#include "PipelineStateCache.h"
#include "ShaderParameterUtils.h"
#include "GlobalShader.h"
#include "GPUSort.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "RHIContext.h"
#include "ShaderParameterMacros.h"

/*------------------------------------------------------------------------------
	Shaders used to generate particle sort keys.
------------------------------------------------------------------------------*/

/** The number of threads per group used to generate particle keys. */
#define PARTICLE_KEY_GEN_THREAD_COUNT 64

/**
 * Uniform buffer parameters for generating particle sort keys.
 */
BEGIN_GLOBAL_SHADER_PARAMETER_STRUCT( FParticleKeyGenParameters, )
	SHADER_PARAMETER( FVector4f, ViewOrigin )
	SHADER_PARAMETER( uint32, ChunksPerGroup )
	SHADER_PARAMETER( uint32, ExtraChunkCount )
	SHADER_PARAMETER( uint32, OutputOffset )
	SHADER_PARAMETER( uint32, EmitterKey )
	SHADER_PARAMETER( uint32, KeyCount )
END_GLOBAL_SHADER_PARAMETER_STRUCT()

IMPLEMENT_GLOBAL_SHADER_PARAMETER_STRUCT(FParticleKeyGenParameters,"ParticleKeyGen");

typedef TUniformBufferRef<FParticleKeyGenParameters> FParticleKeyGenUniformBufferRef;

/**
 * Compute shader used to generate particle sort keys.
 */
class FParticleSortKeyGenCS : public FGlobalShader
{
	DECLARE_SHADER_TYPE(FParticleSortKeyGenCS,Global);

public:

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment( Parameters, OutEnvironment );
		OutEnvironment.SetDefine( TEXT("THREAD_COUNT"), PARTICLE_KEY_GEN_THREAD_COUNT );
		OutEnvironment.SetDefine( TEXT("TEXTURE_SIZE_X"), GParticleSimulationTextureSizeX );
		OutEnvironment.SetDefine( TEXT("TEXTURE_SIZE_Y"), GParticleSimulationTextureSizeY );
	}

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		// this shader writes to a 2 component RWBuffer, which is not supported in GLES
		return !IsMobilePlatform(Parameters.Platform);
	}

	/** Default constructor. */
	FParticleSortKeyGenCS()
	{
	}

	/** Initialization constructor. */
	explicit FParticleSortKeyGenCS( const ShaderMetaType::CompiledShaderInitializerType& Initializer )
		: FGlobalShader(Initializer)
	{
		InParticleIndices.Bind( Initializer.ParameterMap, TEXT("InParticleIndices") );
		PositionTexture.Bind( Initializer.ParameterMap, TEXT("PositionTexture") );
		PositionTextureSampler.Bind( Initializer.ParameterMap, TEXT("PositionTextureSampler") );
		OutKeys.Bind( Initializer.ParameterMap, TEXT("OutKeys") );
		OutParticleIndices.Bind( Initializer.ParameterMap, TEXT("OutParticleIndices") );
	}

	void SetParameters(
		FRHIBatchedShaderParameters& BatchedParameters,
		const FParticleKeyGenParameters& KeyGenParameters,
		FRHIUnorderedAccessView* OutKeysUAV,
		FRHIUnorderedAccessView* OutIndicesUAV,
		FRHITexture* PositionTextureRHI,
		FRHIShaderResourceView* InIndicesSRV
		)
	{
		FParticleKeyGenUniformBufferRef KeyGenUniformBuffer = FParticleKeyGenUniformBufferRef::CreateUniformBufferImmediate(KeyGenParameters, UniformBuffer_SingleDraw);
		SetUniformBufferParameter(BatchedParameters, GetUniformBufferParameter<FParticleKeyGenParameters>(), KeyGenUniformBuffer);

		SetUAVParameter(BatchedParameters, OutKeys, OutKeysUAV);
		SetUAVParameter(BatchedParameters, OutParticleIndices, OutIndicesUAV);
		SetTextureParameter(BatchedParameters, PositionTexture, PositionTextureRHI);
		SetSRVParameter(BatchedParameters, InParticleIndices, InIndicesSRV);
	}

	void UnsetParameters(FRHIBatchedShaderUnbinds& BatchedUnbinds)
	{
		UnsetSRVParameter(BatchedUnbinds, InParticleIndices);
		UnsetUAVParameter(BatchedUnbinds, OutKeys);
		UnsetUAVParameter(BatchedUnbinds, OutParticleIndices);
	}

private:
	/** Input buffer containing particle indices. */
	LAYOUT_FIELD(FShaderResourceParameter, InParticleIndices);
	/** Texture containing particle positions. */
	LAYOUT_FIELD(FShaderResourceParameter, PositionTexture);
	LAYOUT_FIELD(FShaderResourceParameter, PositionTextureSampler);
	/** Output key buffer. */
	LAYOUT_FIELD(FShaderResourceParameter, OutKeys);
	/** Output indices buffer. */
	LAYOUT_FIELD(FShaderResourceParameter, OutParticleIndices);
};
IMPLEMENT_SHADER_TYPE(,FParticleSortKeyGenCS,TEXT("/Engine/Private/ParticleSortKeyGen.usf"),TEXT("GenerateParticleSortKeys"),SF_Compute);

/**
 * Generate sort keys for a list of particles.
 * @param KeyBufferUAV - Unordered access view of the buffer where sort keys will be stored.
 * @param SortedVertexBufferUAV - Unordered access view of the vertex buffer where particle indices will be stored.
 * @param PositionTextureRHI - The texture containing world space positions for all particles.
 * @param SimulationsToSort - A list of simulations to generate sort keys for.
 * @returns the total number of particles being sorted.
 */
int32 GenerateParticleSortKeys(
	FRHICommandListImmediate& RHICmdList,
	FRHIUnorderedAccessView* KeyBufferUAV,
	FRHIUnorderedAccessView* SortedVertexBufferUAV,
	FRHITexture* PositionTextureRHI,
	const TArray<FParticleSimulationSortInfo>& SimulationsToSort,
	ERHIFeatureLevel::Type FeatureLevel,
	int32 BatchId
	)
{
	check(FeatureLevel >= ERHIFeatureLevel::SM5);

	FParticleKeyGenParameters KeyGenParameters;
	const uint32 MaxGroupCount = 128;
	int32 TotalParticleCount = 0;

	// Grab the shader, set output.
	TShaderMapRef<FParticleSortKeyGenCS> KeyGenCS(GetGlobalShaderMap(FeatureLevel));
	SetComputePipelineState(RHICmdList, KeyGenCS.GetComputeShader());


	// TR-KeyGen : No sync needed between tasks since they update different parts of the data (assuming it's ok if cache line overlap).
	RHICmdList.BeginUAVOverlap({ KeyBufferUAV, SortedVertexBufferUAV });

	// For each simulation, generate keys and store them in the sorting buffers.
	for (const FParticleSimulationSortInfo& SortInfo : SimulationsToSort)
	{
		if (SortInfo.AllocationInfo.SortBatchId == BatchId)
		{
			// Create the uniform buffer.
			const uint32 ParticleCount = SortInfo.ParticleCount;
			const uint32 AlignedParticleCount = ((ParticleCount + PARTICLE_KEY_GEN_THREAD_COUNT - 1) & (~(PARTICLE_KEY_GEN_THREAD_COUNT - 1)));
			const uint32 ChunkCount = AlignedParticleCount / PARTICLE_KEY_GEN_THREAD_COUNT;
			const uint32 GroupCount = FMath::Clamp<uint32>( ChunkCount, 1, MaxGroupCount );
			KeyGenParameters.ViewOrigin = FVector3f(SortInfo.ViewOrigin); // LWC_TODO: precision loss
			KeyGenParameters.ChunksPerGroup = ChunkCount / GroupCount;
			KeyGenParameters.ExtraChunkCount = ChunkCount % GroupCount;
			KeyGenParameters.OutputOffset = SortInfo.AllocationInfo.BufferOffset;
			KeyGenParameters.EmitterKey = (uint32)SortInfo.AllocationInfo.ElementIndex << 16;
			KeyGenParameters.KeyCount = ParticleCount;

			SetShaderParametersLegacyCS(RHICmdList, KeyGenCS, KeyGenParameters, KeyBufferUAV, SortedVertexBufferUAV, PositionTextureRHI, SortInfo.VertexBufferSRV);

			DispatchComputeShader(RHICmdList, KeyGenCS.GetShader(), GroupCount, 1, 1);
		}
	}

	// Clear the output buffer.
	UnsetShaderParametersLegacyCS(RHICmdList, KeyGenCS);

	RHICmdList.EndUAVOverlap({ KeyBufferUAV, SortedVertexBufferUAV });

	return TotalParticleCount;
}

/*------------------------------------------------------------------------------
	Buffers used to hold sorting results.
------------------------------------------------------------------------------*/

/**
 * Initialize RHI resources.
 */
void FParticleSortBuffers::InitRHI(FRHICommandListBase& RHICmdList)
{
	for (int32 BufferIndex = 0; BufferIndex < 2; ++BufferIndex)
	{
		{
			const FRHIBufferCreateDesc CreateDesc =
				FRHIBufferCreateDesc::CreateVertex<uint32>(TEXT("PartialSortKeyBuffer"), BufferSize)
				.AddUsage(EBufferUsageFlags::Static | EBufferUsageFlags::ShaderResource | EBufferUsageFlags::UnorderedAccess)
				.DetermineInitialState();

			KeyBuffers[BufferIndex] = RHICmdList.CreateBuffer(CreateDesc);
			KeyBufferSRVs[BufferIndex] = RHICmdList.CreateShaderResourceView(
				KeyBuffers[BufferIndex],
				FRHIViewDesc::CreateBufferSRV()
					.SetType(FRHIViewDesc::EBufferType::Typed)
					.SetFormat(PF_R32_UINT));
			KeyBufferUAVs[BufferIndex] = RHICmdList.CreateUnorderedAccessView(
				KeyBuffers[BufferIndex],
				FRHIViewDesc::CreateBufferUAV()
					.SetType(FRHIViewDesc::EBufferType::Typed)
					.SetFormat(PF_R32_UINT));
		}

		{
			const FRHIBufferCreateDesc CreateDesc =
				FRHIBufferCreateDesc::CreateVertex<uint32>(TEXT("PartialSortVertexBuffer"), BufferSize)
				.AddUsage(EBufferUsageFlags::Static | EBufferUsageFlags::ShaderResource | EBufferUsageFlags::UnorderedAccess)
				.DetermineInitialState();

			VertexBuffers[BufferIndex] = RHICmdList.CreateBuffer(CreateDesc);
			VertexBufferSortSRVs[BufferIndex] = RHICmdList.CreateShaderResourceView(
				VertexBuffers[BufferIndex],
				FRHIViewDesc::CreateBufferSRV()
					.SetType(FRHIViewDesc::EBufferType::Typed)
					.SetFormat(PF_R32_UINT));
			VertexBufferSortUAVs[BufferIndex] = RHICmdList.CreateUnorderedAccessView(
				VertexBuffers[BufferIndex],
				FRHIViewDesc::CreateBufferUAV()
					.SetType(FRHIViewDesc::EBufferType::Typed)
					.SetFormat(PF_R32_UINT));
		}
	}
}

/**
 * Release RHI resources.
 */
void FParticleSortBuffers::ReleaseRHI()
{
	for ( int32 BufferIndex = 0; BufferIndex < 2; ++BufferIndex )
	{
		KeyBufferUAVs[BufferIndex].SafeRelease();
		KeyBufferSRVs[BufferIndex].SafeRelease();
		KeyBuffers[BufferIndex].SafeRelease();

		VertexBufferSortUAVs[BufferIndex].SafeRelease();
		VertexBufferSortSRVs[BufferIndex].SafeRelease();
		VertexBuffers[BufferIndex].SafeRelease();
	}
}

/**
 * Retrieve buffers needed to sort on the GPU.
 */
FGPUSortBuffers FParticleSortBuffers::GetSortBuffers()
{
	FGPUSortBuffers SortBuffers;
		 
	for ( int32 BufferIndex = 0; BufferIndex < 2; ++BufferIndex )
	{
		SortBuffers.RemoteKeySRVs[BufferIndex] = KeyBufferSRVs[BufferIndex];
		SortBuffers.RemoteKeyUAVs[BufferIndex] = KeyBufferUAVs[BufferIndex];
		SortBuffers.RemoteValueSRVs[BufferIndex] = VertexBufferSortSRVs[BufferIndex];
		SortBuffers.RemoteValueUAVs[BufferIndex] = VertexBufferSortUAVs[BufferIndex];
	}

	return SortBuffers;
}
