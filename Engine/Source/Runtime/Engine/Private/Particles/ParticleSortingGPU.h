// Copyright Epic Games, Inc. All Rights Reserved.

/*==============================================================================
	ParticleSortingGPU.h: Interface for sorting GPU particles.
==============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "RenderResource.h"
#include "GPUSortManager.h"

/**
 * The information required to sort particles belonging to an individual simulation.
 */
struct FParticleSimulationSortInfo
{
	typedef FGPUSortManager::FAllocationInfo FAllocationInfo;

	FParticleSimulationSortInfo(FRHIShaderResourceView* InVertexBufferSRV, const FVector& InViewOrigin, uint32 InParticleCount, const FAllocationInfo& InAllocationInfo) 
		: VertexBufferSRV(InVertexBufferSRV)
		, ViewOrigin(InViewOrigin) 
		, ParticleCount(InParticleCount) 
		, AllocationInfo(InAllocationInfo)
	{}

	/** Vertex buffer containing indices in to the particle state texture. */
	FRHIShaderResourceView* VertexBufferSRV = nullptr;
	/** World space position from which to sort. */
	FVector ViewOrigin;
	/** The number of particles in the simulation. */
	uint32 ParticleCount = 0;

	// The GPUSortManager bindings for this sort task.
	FGPUSortManager::FAllocationInfo AllocationInfo;
};

int32 GenerateParticleSortKeys(
	FRHICommandListImmediate& RHICmdList,
	FRHIUnorderedAccessView* KeyBufferUAV,
	FRHIUnorderedAccessView* SortedVertexBufferUAV,
	FRHITexture* PositionTextureRHI,
	const TArray<FParticleSimulationSortInfo>& SimulationsToSort,
	ERHIFeatureLevel::Type FeatureLevel,
	int32 BatchId = INDEX_NONE);

