// Copyright Epic Games, Inc. All Rights Reserved.

#include "Particles/ParticleSystem.h"
#include "DeviceProfiles/DeviceProfile.h"
#include "DeviceProfiles/DeviceProfileManager.h"
#include "Interfaces/ITargetPlatform.h"
#include "ObjectEditorUtils.h"
#include "ParticleHelper.h"
#include "Particles/Collision/ParticleModuleCollision.h"
#include "Particles/Color/ParticleModuleColorBase.h"
#include "Particles/ParticleEmitter.h"
#include "Particles/ParticleLODLevel.h"
#include "Particles/ParticleModuleRequired.h"
#include "Particles/ParticleSystemComponent.h"
#include "Particles/Spawn/ParticleModuleSpawn.h"
#include "Particles/TypeData/ParticleModuleTypeDataBase.h"
#include "Particles/TypeData/ParticleModuleTypeDataGpu.h"
#include "PSOPrecacheMaterial.h"
#include "UObject/AssetRegistryTagsContext.h"
#include "UObject/ObjectSaveContext.h"
#include "UObject/UObjectIterator.h"
#include "PSOPrecacheSettings.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ParticleSystem)

////////////////////////////////////////////////////////////////////////////////////////////////////

UFXSystemAsset::UFXSystemAsset(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

void UFXSystemAsset::LaunchPSOPrecachingInternal(const FMaterialInterfacePSOPrecacheParamsList& PSOPrecacheParamsList)
{
	FGraphEventArray PrecachePSOsEvents;
	if (IsComponentPSOPrecachingEnabled())
	{
		for (const FMaterialInterfacePSOPrecacheParams& MaterialPSOPrecacheParams : PSOPrecacheParamsList)
		{
			if (MaterialPSOPrecacheParams.MaterialInterface)
			{
				FGraphEventArray MaterialPSOsEvents = MaterialPSOPrecacheParams.MaterialInterface->PrecachePSOs(MaterialPSOPrecacheParams.VertexFactoryDataList, MaterialPSOPrecacheParams.PSOPrecacheParams, MaterialPSOPrecacheParams.Priority, MaterialPSOPrecacheRequestIDs);
				if (!MaterialPSOsEvents.IsEmpty())
				{
					MaterialPSORequests.Add(MaterialPSOPrecacheParams.MaterialInterface, MaterialPSOsEvents);
					PrecachePSOsEvents.Append(MaterialPSOsEvents);
				}
			}
		}
	} 

	// Create task to signal that the PSO precache events are done by adding them as prerequisite to the task.
	if (!PrecachePSOsEvents.IsEmpty())
	{
		struct FReleasePrecachePSOsEventTask
		{
			explicit FReleasePrecachePSOsEventTask(UFXSystemAsset* OwnerAsset)
				: WeakOwnerAsset(OwnerAsset)
			{
			}

			static TStatId GetStatId() { return TStatId(); }
			static ENamedThreads::Type GetDesiredThread() { return ENamedThreads::GameThread; }
			static ESubsequentsMode::Type GetSubsequentsMode() { return ESubsequentsMode::TrackSubsequents; }

			void DoTask(ENamedThreads::Type CurrentThread, const FGraphEventRef& MyCompletionGraphEvent)
			{
				if (UFXSystemAsset* Asset = WeakOwnerAsset.Get())
				{
					Asset->PrecachePSOsEvent = nullptr;
					Asset->MaterialPSORequests.Empty();
				}
			}

			TWeakObjectPtr<UFXSystemAsset> WeakOwnerAsset;
		};

		// need to set `PrecachePSOsEvent` before the task is launched to not race with its execution
		TGraphTask<FReleasePrecachePSOsEventTask>* ReleasePrecachePSOsEventTask = TGraphTask<FReleasePrecachePSOsEventTask>::CreateTask(&PrecachePSOsEvents).ConstructAndHold(this);
		PrecachePSOsEvent = ReleasePrecachePSOsEventTask->GetCompletionEvent();
		ReleasePrecachePSOsEventTask->Unlock();
	}

	PSOPrecachingLaunched = true;
}

bool UFXSystemAsset::IsLoadingPSOCache(UMaterialInterface* Material) const
{
	if (!PSOPrecachingLaunched || MaterialPSORequests.IsEmpty() || !MaterialPSORequests.Contains(Material))
	{
		return false;
	}
	for (const FGraphEventRef& Task : MaterialPSORequests[Material])
	{
		if (Task && !Task->IsCompleted())
		{
			return true;
		}
	}
	return false;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

UParticleSystem::UParticleSystem(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, OcclusionBoundsMethod(EPSOBM_ParticleBounds)
	, bAnyEmitterLoopsForever(false)
	, HighestSignificance(EParticleSignificanceLevel::Critical)
	, LowestSignificance(EParticleSignificanceLevel::Low)
	, bShouldManageSignificance(false)
	, bIsImmortal(false)
	, bWillBecomeZombie(false)
{
#if WITH_EDITORONLY_DATA
	ThumbnailDistance = 200.0;
	ThumbnailWarmup = 1.0;
#endif // WITH_EDITORONLY_DATA
	UpdateTime_FPS = 60.0f;
	UpdateTime_Delta = 1.0f/60.0f;
	WarmupTime = 0.0f;
	WarmupTickRate = 0.0f;
#if WITH_EDITORONLY_DATA
	EditorLODSetting = 0;
#endif // WITH_EDITORONLY_DATA
	FixedRelativeBoundingBox.Min = FVector(-1.0f, -1.0f, -1.0f);
	FixedRelativeBoundingBox.Max = FVector(1.0f, 1.0f, 1.0f);
	FixedRelativeBoundingBox.IsValid = true;

	LODMethod = PARTICLESYSTEMLODMETHOD_Automatic;
	LODDistanceCheckTime = 0.25f;
	bRegenerateLODDuplicate = false;
	ThumbnailImageOutOfDate = true;
#if WITH_EDITORONLY_DATA
	FloorMesh = TEXT("/Engine/EditorMeshes/AnimTreeEd_PreviewFloor.AnimTreeEd_PreviewFloor");
	FloorPosition = FVector(0.0f, 0.0f, 0.0f);
	FloorRotation = FRotator(0.0f, 0.0f, 0.0f);
	FloorScale = 1.0f;
	FloorScale3D = FVector(1.0f, 1.0f, 1.0f);
#endif // WITH_EDITORONLY_DATA

	MacroUVPosition = FVector(0.0f, 0.0f, 0.0f);

	MacroUVRadius = 200.0f;
	bAutoDeactivate = true;
	MinTimeBetweenTicks = 0;
	InsignificantReaction = EParticleSystemInsignificanceReaction::Auto;
	InsignificanceDelay = 0.0f;
	MaxSignificanceLevel = EParticleSignificanceLevel::Critical;
	MaxPoolSize = 32;


	bAllowManagedTicking = true;
}

ParticleSystemLODMethod UParticleSystem::GetCurrentLODMethod()
{
	return ParticleSystemLODMethod(LODMethod);
}


int32 UParticleSystem::GetLODLevelCount()
{
	return LODDistances.Num();
}


float UParticleSystem::GetLODDistance(int32 LODLevelIndex)
{
	if (LODLevelIndex >= LODDistances.Num())
	{
		return -1.0f;
	}

	return LODDistances[LODLevelIndex];
}

void UParticleSystem::SetCurrentLODMethod(ParticleSystemLODMethod InMethod)
{
	LODMethod = InMethod;
}


bool UParticleSystem::SetLODDistance(int32 LODLevelIndex, float InDistance)
{
	if (LODLevelIndex >= LODDistances.Num())
	{
		return false;
	}

	LODDistances[LODLevelIndex] = InDistance;

	return true;
}

bool UParticleSystem::DoesAnyEmitterHaveMotionBlur(int32 LODLevelIndex) const
{
	for (auto& EmitterIter : Emitters)
	{
		if (EmitterIter)
		{
			auto* EmitterLOD = EmitterIter->GetLODLevel(LODLevelIndex);
			if (!EmitterLOD)
			{
				continue;
			}

			if (EmitterLOD->TypeDataModule && EmitterLOD->TypeDataModule->IsMotionBlurEnabled())
			{
				return true;
			}

			if (EmitterLOD->RequiredModule && EmitterLOD->RequiredModule->ShouldUseVelocityForMotionBlur())
			{
				return true;
			}
		}
	}

	return false;
}

#if WITH_EDITOR
void UParticleSystem::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	UpdateTime_Delta = 1.0f / UpdateTime_FPS;

	bIsElligibleForAsyncTickComputed = false;

	//If the property is NULL then we don't really know what's happened. 
	//Could well be a module change, requiring all instances to be destroyed and recreated.
	bool bEmptyInstances = PropertyChangedEvent.Property == NULL;
	for (TObjectIterator<UParticleSystemComponent> It;It;++It)
	{
		if (It->Template == this)
		{
			It->UpdateInstances(bEmptyInstances);
		}
	}

	// Ensure the bounds have a positive size
	if (FixedRelativeBoundingBox.IsValid)
	{
		if (FixedRelativeBoundingBox.Min.X > FixedRelativeBoundingBox.Max.X)
		{
			Swap(FixedRelativeBoundingBox.Min.X, FixedRelativeBoundingBox.Max.X);
		}
		if (FixedRelativeBoundingBox.Min.Y > FixedRelativeBoundingBox.Max.Y)
		{
			Swap(FixedRelativeBoundingBox.Min.Y, FixedRelativeBoundingBox.Max.Y);
		}
		if (FixedRelativeBoundingBox.Min.Z > FixedRelativeBoundingBox.Max.Z)
		{
			Swap(FixedRelativeBoundingBox.Min.Z, FixedRelativeBoundingBox.Max.Z);
		}
	}

	// Recompute the looping flag
	bAnyEmitterLoopsForever = false;
	bIsImmortal = false;
	bWillBecomeZombie = false;
	HighestSignificance = EParticleSignificanceLevel::Low;
	LowestSignificance = EParticleSignificanceLevel::Critical;
	for (UParticleEmitter* Emitter : Emitters)
	{
		if (Emitter != nullptr)
		{
			for (const UParticleLODLevel* LODLevel : Emitter->LODLevels)
			{
				if (LODLevel != nullptr)
				{
					if (LODLevel->bEnabled)
					{
						const UParticleModuleRequired* RequiredModule = LODLevel->RequiredModule;
						if ((RequiredModule != nullptr) && (RequiredModule->EmitterLoops == 0))
						{
							bAnyEmitterLoopsForever = true;

							UParticleModuleSpawn* SpawnModule = LODLevel->SpawnModule;
							check(SpawnModule);

							// check if any emitter will cause the system to never be deleted
							// terms here are zombie (burst-only, so will stop spawning but emitter instances and psys component will continue existing)
							// and immortal (any emitter will loop indefinitely and does not have finite duration)
							if (RequiredModule->EmitterDuration == 0.0f)
							{
								bIsImmortal = true;
								if (SpawnModule->GetMaximumSpawnRate() == 0.0f && !bAutoDeactivate)
								{
									bWillBecomeZombie = true;
								}
							}
						}
					}
				}
			}

			EParticleSignificanceLevel EmitterSignificance = FMath::Min(MaxSignificanceLevel, Emitter->SignificanceLevel);
			if (EmitterSignificance > HighestSignificance)
			{
				HighestSignificance = EmitterSignificance;
			}
			if (EmitterSignificance < LowestSignificance)
			{
				LowestSignificance = EmitterSignificance;
			}
		}
	}

	bShouldManageSignificance = GetLowestSignificance() != EParticleSignificanceLevel::Critical/* && !ContainsEmitterType(UParticleModuleTypeDataBeam2::StaticClass())*/;

	//cap the WarmupTickRate to realistic values
	if (WarmupTickRate <= 0)
	{
		WarmupTickRate = 0;
	}
	else if (WarmupTickRate > WarmupTime)
	{
		WarmupTickRate = WarmupTime;
	}

	ThumbnailImageOutOfDate = true;

	Super::PostEditChangeProperty(PropertyChangedEvent);
}
#endif // WITH_EDITOR

void UParticleSystem::PreSave(FObjectPreSaveContext ObjectSaveContext)
{
	Super::PreSave(ObjectSaveContext);
#if WITH_EDITORONLY_DATA
	// Ensure that soloing is undone...
	int32 NumEmitters = FMath::Min(Emitters.Num(),SoloTracking.Num());
	for (int32 EmitterIdx = 0; EmitterIdx < NumEmitters; EmitterIdx++)
	{
		UParticleEmitter* Emitter = Emitters[EmitterIdx];
		if (Emitter != nullptr)
		{
			Emitter->bIsSoloing = false;
			FLODSoloTrack& SoloTrack = SoloTracking[EmitterIdx];
			int32 NumLODs = FMath::Min(Emitter->LODLevels.Num(), SoloTrack.SoloEnableSetting.Num());
			for (int32 LODIdx = 0; LODIdx < NumLODs; LODIdx++)
			{
				UParticleLODLevel* LODLevel = Emitter->LODLevels[LODIdx];
				{
					// Restore the enabled settings - ie turn off soloing...
					LODLevel->bEnabled = SoloTrack.SoloEnableSetting[LODIdx];
				}
			}
		}
	}
#endif // WITH_EDITORONLY_DATA
}

bool UParticleSystem::IsPostLoadThreadSafe() const
{
	return false;
}

void UParticleSystem::PostLoad()
{
	Super::PostLoad();

	// Run thru all of the emitters, load them up and compute some flags based on them
	bHasPhysics = false;
	bAnyEmitterLoopsForever = false;
	bIsImmortal = false;
	bWillBecomeZombie = false;
	HighestSignificance = EParticleSignificanceLevel::Low;
	LowestSignificance = EParticleSignificanceLevel::Critical;
	for (int32 i = Emitters.Num() - 1; i >= 0; i--)
	{
		// Remove any old emitters
		UParticleEmitter* Emitter = Emitters[i];
		if (Emitter == NULL)
		{
			// Empty emitter slots are ok with cooked content.
			if( !FPlatformProperties::RequiresCookedData() && !GIsServer)
			{
				UE_LOGF(LogParticles, Warning, "ParticleSystem contains empty emitter slots - %ls", *GetFullName());
			}
			continue;
		}

		Emitter->ConditionalPostLoad();

		bool bCookedOut = false;
		if (UParticleSpriteEmitter* SpriteEmitter = Cast<UParticleSpriteEmitter>(Emitter))
		{
			bCookedOut = SpriteEmitter->bCookedOut;
		}

		if (!bCookedOut)
		{
			if (Emitter->LODLevels.Num() == 0)
			{
				UE_LOGF(LogParticles, Warning, "ParticleSystem contains emitter with no lod levels - %ls - %ls", *GetFullName(), *Emitter->GetFullName());
				continue;
			}

			UParticleLODLevel* LODLevel = Emitter->LODLevels[0];
			check(LODLevel);

			LODLevel->ConditionalPostLoad();
				
			//@todo. Move these flag calculations into the editor and serialize?
			//Should mirror implementation in PostEditChangeProperty.
			for (UParticleLODLevel* ParticleLODLevel : Emitter->LODLevels)
			{
				if (ParticleLODLevel)
				{
					//@todo. This is a temporary fix for emitters that apply physics.
					// Check for collision modules with bApplyPhysics set to true
					for (int32 ModuleIndex = 0; ModuleIndex < LODLevel->Modules.Num(); ModuleIndex++)
					{
						UParticleModuleCollision* CollisionModule = Cast<UParticleModuleCollision>(ParticleLODLevel->Modules[ModuleIndex]);
						if (CollisionModule)
						{
							if (CollisionModule->bApplyPhysics == true)
							{
								bHasPhysics = true;
								break;
							}
						}
					}

					if (LODLevel->bEnabled)
					{
						const UParticleModuleRequired* RequiredModule = LODLevel->RequiredModule;
						if ((RequiredModule != nullptr) && (RequiredModule->EmitterLoops == 0))
						{
							bAnyEmitterLoopsForever = true;

							UParticleModuleSpawn* SpawnModule = LODLevel->SpawnModule;
							check(SpawnModule);

							if (RequiredModule->EmitterDuration == 0.0f)
							{
								bIsImmortal = true;
								if (SpawnModule->GetMaximumSpawnRate() == 0.0f && !bAutoDeactivate)
								{
									bWillBecomeZombie = true;
								}
							}
						}
					}
				}
			}

			EParticleSignificanceLevel EmitterSignificance = FMath::Min(MaxSignificanceLevel, Emitter->SignificanceLevel);
			if (EmitterSignificance > HighestSignificance)
			{
				HighestSignificance = EmitterSignificance;
			}
			if (EmitterSignificance < LowestSignificance)
			{
				LowestSignificance = EmitterSignificance;
			}
		}
	}

	bShouldManageSignificance = GetLowestSignificance() != EParticleSignificanceLevel::Critical /* && !ContainsEmitterType(UParticleModuleTypeDataBeam2::StaticClass())*/;

	if (LODSettings.Num() == 0)
	{
		if (Emitters.Num() > 0)
		{
			UParticleEmitter* Emitter = Emitters[0];
			if (Emitter)
			{
				LODSettings.AddUninitialized(Emitter->LODLevels.Num());
				for (int32 LODIndex = 0; LODIndex < LODSettings.Num(); LODIndex++)
				{
					LODSettings[LODIndex] = FParticleSystemLOD::CreateParticleSystemLOD();
				}
			}
		}
		else
		{
			LODSettings.AddUninitialized();
			LODSettings[0] = FParticleSystemLOD::CreateParticleSystemLOD();
		}
	}

	// Add default LOD Distances
	if( LODDistances.Num() == 0 && Emitters.Num() > 0 )
	{
		UParticleEmitter* Emitter = Emitters[0];
		if (Emitter)
		{
			LODDistances.AddUninitialized(Emitter->LODLevels.Num());
			for (int32 LODIndex = 0; LODIndex < LODDistances.Num(); LODIndex++)
			{
				LODDistances[LODIndex] = LODIndex * 2500.0f;
			}
		}
	}

	if (UPSOPrecacheSettingsManager::GetSettings().ParticlePrecachingTime >= 3)
	{
		PrecacheAssetPSOs();
	}

#if WITH_EDITOR
	// Due to there still being some ways that LODLevel counts get mismatched,
	// when loading in the editor LOD levels will always be checked and fixed
	// up... This can be removed once all the edge cases that lead to the
	// problem are found and fixed.
	if (GIsEditor)
	{
		// Fix the LOD distance array and mismatched lod levels
		int32 LODCount_0 = -1;
		for (int32 EmitterIndex = 0; EmitterIndex < Emitters.Num(); EmitterIndex++)
		{
			UParticleEmitter* Emitter  = Emitters[EmitterIndex];
			if (Emitter)
			{
				if (LODCount_0 == -1)
				{
					LODCount_0 = Emitter->LODLevels.Num();
				}
				else
				{
					int32 EmitterLODCount = Emitter->LODLevels.Num();
					if (EmitterLODCount != LODCount_0)
					{
						UE_LOGF(LogParticles, Warning, "Emitter %d has mismatched LOD level count - expected %d, found %d. PS = %ls",
							EmitterIndex, LODCount_0, EmitterLODCount, *GetPathName());
						UE_LOGF(LogParticles, Warning, "Fixing up now... Package = %ls", *(GetOutermost()->GetPathName()));

						if (EmitterLODCount > LODCount_0)
						{
							Emitter->LODLevels.RemoveAt(LODCount_0, EmitterLODCount - LODCount_0);
						}
						else
						{
							for (int32 NewLODIndex = EmitterLODCount; NewLODIndex < LODCount_0; NewLODIndex++)
							{
								if (Emitter->CreateLODLevel(NewLODIndex) != NewLODIndex)
								{
									UE_LOGF(LogParticles, Warning, "Failed to add LOD level %d", NewLODIndex);
								}
							}
						}
					}
				}
			}
		}

		if (LODCount_0 > 0)
		{
			if (LODDistances.Num() < LODCount_0)
			{
				for (int32 DistIndex = LODDistances.Num(); DistIndex < LODCount_0; DistIndex++)
				{
					float Distance = DistIndex * 2500.0f;
					LODDistances.Add(Distance);
				}
			}
			else
			if (LODDistances.Num() > LODCount_0)
			{
				LODDistances.RemoveAt(LODCount_0, LODDistances.Num() - LODCount_0);
			}
		}
		else
		{
			LODDistances.Empty();
		}

		if (LODCount_0 > 0)
		{
			if (LODSettings.Num() < LODCount_0)
			{
				for (int32 DistIndex = LODSettings.Num(); DistIndex < LODCount_0; DistIndex++)
				{
					LODSettings.Add(FParticleSystemLOD::CreateParticleSystemLOD());
				}
			}
			else
				if (LODSettings.Num() > LODCount_0)
				{
					LODSettings.RemoveAt(LODCount_0, LODSettings.Num() - LODCount_0);
				}
		}
		else
		{
			LODSettings.Empty();
		}
	}
#endif

#if WITH_EDITORONLY_DATA
	// Reset cascade's UI LOD setting to 0.
	EditorLODSetting = 0;
#endif // WITH_EDITORONLY_DATA

	FixedRelativeBoundingBox.IsValid = true;

	// Set up the SoloTracking...
	SetupSoloing();
}

void UParticleSystem::PrecacheAssetPSOs()
{
	if (HasLaunchedPSOPrecaching() || (!IsComponentPSOPrecachingEnabled() && !IsResourcePSOPrecachingEnabled()))
	{
		return;
	}

	FMaterialInterfacePSOPrecacheParamsList PSOPrecacheParamsList;

	FMaterialInterfacePSOPrecacheParams NewEntry;
	NewEntry.PSOPrecacheParams.SetMobility(EComponentMobility::Movable);

	// No per component emitter materials known at this point in time
	TArray<UMaterialInterface*> EmptyEmitterMaterials;
	// Cached array to collect all materials used for LOD level
	TArray<UMaterialInterface*> Materials;

	for (int32 EmitterIdx = 0; EmitterIdx < Emitters.Num(); ++EmitterIdx)
	{
		const UParticleEmitter* Emitter = Emitters[EmitterIdx];
		if (!Emitter)
		{
			continue;
		}

		for (int32 LodIndex = 0; LodIndex < Emitter->LODLevels.Num(); ++LodIndex)
		{
			const UParticleLODLevel* LOD = Emitter->LODLevels[LodIndex];
			if (LOD && LOD->bEnabled)
			{
				UParticleModuleTypeDataBase::FPSOPrecacheParams PrecacheParams;
				if (LOD->TypeDataModule)
				{
					LOD->TypeDataModule->CollectPSOPrecacheData(Emitter, PrecacheParams);
				}
				else
				{
					bool bUsesDynamicParameter = (Emitter->DynamicParameterDataOffset > 0);
					FPSOPrecacheVertexFactoryData VFData;
					VFData.VertexFactoryType = &FParticleSpriteVertexFactory::StaticType;
					VFData.CustomDefaultVertexDeclaration = FParticleSpriteVertexFactory::GetPSOPrecacheVertexDeclaration(bUsesDynamicParameter);
					PrecacheParams.VertexFactoryDataList.Add(VFData);
					PrecacheParams.PrimitiveType = PT_TriangleList;
				}

				Materials.Empty();
				LOD->GetUsedMaterials(Materials, NamedMaterialSlots, EmptyEmitterMaterials);

				for (UMaterialInterface* MaterialInterface : Materials)
				{
					NewEntry.MaterialInterface = MaterialInterface;
					NewEntry.VertexFactoryDataList = PrecacheParams.VertexFactoryDataList;
					NewEntry.PSOPrecacheParams.PrimitiveType = PrecacheParams.PrimitiveType;

					AddMaterialInterfacePSOPrecacheParamsToList(NewEntry, PSOPrecacheParamsList);
				}
			}
		}
	}

	LaunchPSOPrecachingInternal(PSOPrecacheParamsList);
}

void UParticleSystem::Serialize(FArchive& Ar)
{
	Ar.UsingCustomVersion(FParticleSystemCustomVersion::GUID);

	int32 CookTargetPlatformDetailModeMask = 0xFFFFFFFF;
#if WITH_EDITOR
	if (Ar.IsCooking())
	{
		// If we're cooking, check the device profile for whether we want to eliminate all emitters that don't match the detail mode. 
		// This will only work if scalability settings affecting detail mode can not be changed at runtime (depends on platform)!
		//
		if (UDeviceProfile* DeviceProfile = UDeviceProfileManager::Get().FindProfile(Ar.CookingTarget()->IniPlatformName()))
		{
			// if we don't prune, we assume all detail modes
			int32 CVarDoPrune = 0;
			if (DeviceProfile->GetConsolidatedCVarValue(TEXT("fx.PruneEmittersOnCookByDetailMode"), CVarDoPrune) && CVarDoPrune == 1)
			{
				// get the detail mode from the device platform ini; if it's not there, we assume all detail modes
				int32 CVarDetailMode = PDM_DefaultValue;
				if (DeviceProfile->GetConsolidatedCVarValue(TEXT("r.DetailMode"), CVarDetailMode))
				{
					CookTargetPlatformDetailModeMask = (1 << CVarDetailMode);
				}
			}
		}

		// if we're cooking, save only emitters with matching detail modes
		for (int32 EmitterIdx = 0; EmitterIdx<Emitters.Num(); EmitterIdx++)
		{
			// null out if detail mode doesn't match
			if (Emitters[EmitterIdx] && !(Emitters[EmitterIdx]->DetailModeBitmask & CookTargetPlatformDetailModeMask))
			{
				UE_LOGF(LogParticles, Display, "Pruning emitter, detail mode mismatch (PDM %i) (only works if platform can't change detail mode at runtime!): %ls - set fx.PruneEmittersOnCookByDetailMode to 0 in DeviceProfile.ini for the target profile to avoid", Emitters[EmitterIdx]->DetailModeBitmask, *Emitters[EmitterIdx]->EmitterName.ToString());
				Emitters[EmitterIdx] = nullptr;
			}
		}
	}
#endif

	Super::Serialize(Ar);

}


void UParticleSystem::UpdateColorModuleClampAlpha(UParticleModuleColorBase* ColorModule)
{
	if (ColorModule)
	{
		TArray<const FCurveEdEntry*> CurveEntries;
		ColorModule->RemoveModuleCurvesFromEditor(CurveEdSetup);
		ColorModule->AddModuleCurvesToEditor(CurveEdSetup, CurveEntries);
	}
}

void UParticleSystem::GetAssetRegistryTags(TArray<FAssetRegistryTag>& OutTags) const
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS;
	Super::GetAssetRegistryTags(OutTags);
	PRAGMA_ENABLE_DEPRECATION_WARNINGS;
}

void UParticleSystem::GetAssetRegistryTags(FAssetRegistryTagsContext Context) const
{
	Context.AddTag( FAssetRegistryTag("HasGPUEmitter", HasGPUEmitter() ? TEXT("True") : TEXT("False"), FAssetRegistryTag::TT_Alphabetical) );

	const float BoundsSize = FixedRelativeBoundingBox.GetSize().GetMax();
	Context.AddTag(FAssetRegistryTag("FixedBoundsSize", bUseFixedRelativeBoundingBox ? FString::Printf(TEXT("%.2f"), BoundsSize) : FString("None"), FAssetRegistryTag::TT_Numerical));

	Context.AddTag(FAssetRegistryTag("NumEmitters", LexToString(Emitters.Num()), FAssetRegistryTag::TT_Numerical));

	Context.AddTag(FAssetRegistryTag("NumLODs", LexToString(LODDistances.Num()), FAssetRegistryTag::TT_Numerical));

	Context.AddTag(FAssetRegistryTag("WarmupTime", LexToString(WarmupTime), FAssetRegistryTag::TT_Numerical));

	// Done here instead of as an AssetRegistrySearchable string to avoid the long prefix on the enum value string
	FString LODMethodString = "Unknown";
	switch (LODMethod)
	{
	case PARTICLESYSTEMLODMETHOD_Automatic:
		LODMethodString = TEXT("Automatic");
		break;
	case PARTICLESYSTEMLODMETHOD_DirectSet:
		LODMethodString = TEXT("DirectSet");
		break;
	case PARTICLESYSTEMLODMETHOD_ActivateAutomatic:
		LODMethodString = TEXT("Activate Automatic");
		break;
	default:
		check(false); // Missing enum entry
		break;
	}
	Context.AddTag(FAssetRegistryTag("LODMethod", LODMethodString, FAssetRegistryTag::TT_Alphabetical));

	Context.AddTag(FAssetRegistryTag("CPUCollision", UsesCPUCollision() ? TEXT("True") : TEXT("False"), FAssetRegistryTag::TT_Alphabetical));
	Context.AddTag(FAssetRegistryTag("Looping", bAnyEmitterLoopsForever ? TEXT("True") : TEXT("False"), FAssetRegistryTag::TT_Alphabetical));
	Context.AddTag(FAssetRegistryTag("Immortal", IsImmortal() ? TEXT("True") : TEXT("False"), FAssetRegistryTag::TT_Alphabetical));
	Context.AddTag(FAssetRegistryTag("Becomes Zombie", WillBecomeZombie() ? TEXT("True") : TEXT("False"), FAssetRegistryTag::TT_Alphabetical));
	Context.AddTag(FAssetRegistryTag("CanBeOccluded", OcclusionBoundsMethod == EParticleSystemOcclusionBoundsMethod::EPSOBM_None ? TEXT("False") : TEXT("True"), FAssetRegistryTag::TT_Alphabetical));

	uint32 NumEmittersAtEachSig[(int32)EParticleSignificanceLevel::Num] = { 0, 0, 0, 0 };
	for (UParticleEmitter* Emitter : Emitters)
	{
		if (Emitter)
		{
			++NumEmittersAtEachSig[(int32)Emitter->SignificanceLevel];			
		}
	}
	Context.AddTag(FAssetRegistryTag("Critical Emitters", LexToString(NumEmittersAtEachSig[(int32)EParticleSignificanceLevel::Critical]), FAssetRegistryTag::TT_Numerical));
	Context.AddTag(FAssetRegistryTag("High Emitters", LexToString(NumEmittersAtEachSig[(int32)EParticleSignificanceLevel::High]), FAssetRegistryTag::TT_Numerical));
	Context.AddTag(FAssetRegistryTag("Medium Emitters", LexToString(NumEmittersAtEachSig[(int32)EParticleSignificanceLevel::Medium]), FAssetRegistryTag::TT_Numerical));
	Context.AddTag(FAssetRegistryTag("Low Emitters", LexToString(NumEmittersAtEachSig[(int32)EParticleSignificanceLevel::Low]), FAssetRegistryTag::TT_Numerical));

	Super::GetAssetRegistryTags(Context);
}

bool UParticleSystem::UsesCPUCollision() const
{
	for (const UParticleEmitter* Emitter : Emitters)
	{
		if (Emitter != nullptr)
		{
			// If we have not yet found a CPU collision module, and we have some enabled LODs to look in..
			if (Emitter->HasAnyEnabledLODs() && Emitter->LODLevels.Num() > 0)
			{
				if (const UParticleLODLevel* HighLODLevel = Emitter->LODLevels[0])
				{
					// Iterate over modules of highest LOD (will have all the modules)
					for (const UParticleModule* Module : HighLODLevel->Modules)
					{
						// If an enabled CPU collision module 
						if (Module->bEnabled && Module->IsA<UParticleModuleCollision>())
						{
							return true;
						}
					}
				}
			}
		}
	}

	return false;
}

bool UParticleSystem::CanBeClusterRoot() const
{
	return true;
}

bool UParticleSystem::CanBePooled()const
{
	if (MaxPoolSize == 0)
	{
		return false;
	}

	return true;
}

bool UParticleSystem::CalculateMaxActiveParticleCounts()
{
	bool bSuccess = true;

	for (int32 EmitterIndex = 0; EmitterIndex < Emitters.Num(); EmitterIndex++)
	{
		UParticleEmitter* Emitter = Emitters[EmitterIndex];
		if (Emitter)
		{
			if (Emitter->CalculateMaxActiveParticleCount() == false)
			{
				bSuccess = false;
			}
		}
	}

	return bSuccess;
}


void UParticleSystem::GetParametersUtilized(TArray<TArray<FString> >& ParticleSysParamList,
											TArray<TArray<FString> >& ParticleParameterList)
{
	ParticleSysParamList.Empty();
	ParticleParameterList.Empty();

	for (int32 EmitterIndex = 0; EmitterIndex < Emitters.Num(); EmitterIndex++)
	{
		int32 CheckIndex;
		CheckIndex = ParticleSysParamList.AddZeroed();
		check(CheckIndex == EmitterIndex);
		CheckIndex = ParticleParameterList.AddZeroed();
		check(CheckIndex == EmitterIndex);

		UParticleEmitter* Emitter = Emitters[EmitterIndex];
		if (Emitter)
		{
			Emitter->GetParametersUtilized(
				ParticleSysParamList[EmitterIndex],
				ParticleParameterList[EmitterIndex]);
		}
	}
}


void UParticleSystem::SetupSoloing()
{
#if WITH_EDITOR
	if (GIsEditor)
	{
		if (Emitters.Num())
		{
			// Store the settings of bEnabled for each LODLevel in each emitter
			SoloTracking.Empty(Emitters.Num());
			SoloTracking.AddZeroed(Emitters.Num());
			for (int32 EmitterIdx = 0; EmitterIdx < Emitters.Num(); EmitterIdx++)
			{
				UParticleEmitter* Emitter = Emitters[EmitterIdx];
				if (Emitter != nullptr)
				{
					FLODSoloTrack& SoloTrack = SoloTracking[EmitterIdx];
					SoloTrack.SoloEnableSetting.Empty(Emitter->LODLevels.Num());
					SoloTrack.SoloEnableSetting.AddZeroed(Emitter->LODLevels.Num());
				}
			}

			for (int32 EmitterIdx = 0; EmitterIdx < Emitters.Num(); EmitterIdx++)
			{
				UParticleEmitter* Emitter = Emitters[EmitterIdx];
				if (Emitter != NULL)
				{
					FLODSoloTrack& SoloTrack = SoloTracking[EmitterIdx];
					int32 MaxLOD = FMath::Min(SoloTrack.SoloEnableSetting.Num(), Emitter->LODLevels.Num());
					for (int32 LODIdx = 0; LODIdx < MaxLOD; LODIdx++)
					{
						UParticleLODLevel* LODLevel = Emitter->LODLevels[LODIdx];
						check(LODLevel);
						SoloTrack.SoloEnableSetting[LODIdx] = LODLevel->bEnabled;
					}
				}
			}
		}
	}
#endif
}


bool UParticleSystem::ToggleSoloing(class UParticleEmitter* InEmitter)
{
	bool bSoloingReturn = false;
	if (InEmitter != NULL)
	{
		bool bOtherEmitterIsSoloing = false;
		// Set the given one
		int32 SelectedIndex = -1;
		for (int32 EmitterIdx = 0; EmitterIdx < Emitters.Num(); EmitterIdx++)
		{
			UParticleEmitter* Emitter = Emitters[EmitterIdx];
			check(Emitter != NULL);
			if (Emitter == InEmitter)
			{
				SelectedIndex = EmitterIdx;
			}
			else
			{
				if (Emitter->bIsSoloing == true)
				{
					bOtherEmitterIsSoloing = true;
					bSoloingReturn = true;
				}
			}
		}

		if (SelectedIndex != -1)
		{
			InEmitter->bIsSoloing = !InEmitter->bIsSoloing;
			for (int32 EmitterIdx = 0; EmitterIdx < Emitters.Num(); EmitterIdx++)
			{
				UParticleEmitter* Emitter = Emitters[EmitterIdx];
				FLODSoloTrack& SoloTrack = SoloTracking[EmitterIdx];
				if (EmitterIdx == SelectedIndex)
				{
					for (int32 LODIdx = 0; LODIdx < InEmitter->LODLevels.Num(); LODIdx++)
					{
						UParticleLODLevel* LODLevel = InEmitter->LODLevels[LODIdx];
						if (InEmitter->bIsSoloing == false)
						{
							if (bOtherEmitterIsSoloing == false)
							{
								// Restore the enabled settings - ie turn off soloing...
								LODLevel->bEnabled = SoloTrack.SoloEnableSetting[LODIdx];
							}
							else
							{
								// Disable the emitter
								LODLevel->bEnabled = false;
							}
						}
						else 
						if (bOtherEmitterIsSoloing == true)
						{
							// Need to restore old settings of this emitter as it is now soloing
							LODLevel->bEnabled = SoloTrack.SoloEnableSetting[LODIdx];
						}
					}
				}
				else
				{
					// Restore all other emitters if this disables soloing...
					if ((InEmitter->bIsSoloing == false) && (bOtherEmitterIsSoloing == false))
					{
						for (int32 LODIdx = 0; LODIdx < Emitter->LODLevels.Num(); LODIdx++)
						{
							UParticleLODLevel* LODLevel = Emitter->LODLevels[LODIdx];
							// Restore the enabled settings - ie turn off soloing...
							LODLevel->bEnabled = SoloTrack.SoloEnableSetting[LODIdx];
						}
					}
					else
					{
						if (Emitter->bIsSoloing == false)
						{
							for (int32 LODIdx = 0; LODIdx < Emitter->LODLevels.Num(); LODIdx++)
							{
								UParticleLODLevel* LODLevel = Emitter->LODLevels[LODIdx];
								// Disable the emitter
								LODLevel->bEnabled = false;
							}
						}
					}
				}
			}
		}

		// We checked the other emitters above...
		// Make sure we catch the case of the first one toggled to true!
		if (InEmitter->bIsSoloing == true)
		{
			bSoloingReturn = true;
		}
	}

	return bSoloingReturn;
}


bool UParticleSystem::TurnOffSoloing()
{
	for (int32 EmitterIdx = 0; EmitterIdx < Emitters.Num(); EmitterIdx++)
	{
		UParticleEmitter* Emitter = Emitters[EmitterIdx];
		if (Emitter != NULL)
		{
			FLODSoloTrack& SoloTrack = SoloTracking[EmitterIdx];
			for (int32 LODIdx = 0; LODIdx < Emitter->LODLevels.Num(); LODIdx++)
			{
				UParticleLODLevel* LODLevel = Emitter->LODLevels[LODIdx];
				if (LODLevel != NULL)
				{
					// Restore the enabled settings - ie turn off soloing...
					LODLevel->bEnabled = SoloTrack.SoloEnableSetting[LODIdx];
				}
			}
			Emitter->bIsSoloing = false;
		}
	}

	return true;
}


void UParticleSystem::SetupLODValidity()
{
	for (int32 EmitterIdx = 0; EmitterIdx < Emitters.Num(); EmitterIdx++)
	{
		UParticleEmitter* Emitter = Emitters[EmitterIdx];
		if (Emitter != NULL)
		{
			for (int32 Pass = 0; Pass < 2; Pass++)
			{
				for (int32 LODIdx = 0; LODIdx < Emitter->LODLevels.Num(); LODIdx++)
				{
					UParticleLODLevel* LODLevel = Emitter->LODLevels[LODIdx];
					if (LODLevel != NULL)
					{
						for (int32 ModuleIdx = -3; ModuleIdx < LODLevel->Modules.Num(); ModuleIdx++)
						{
							int32 ModuleFetchIdx;
							switch (ModuleIdx)
							{
							case -3:	ModuleFetchIdx = INDEX_REQUIREDMODULE;	break;
							case -2:	ModuleFetchIdx = INDEX_SPAWNMODULE;		break;
							case -1:	ModuleFetchIdx = INDEX_TYPEDATAMODULE;	break;
							default:	ModuleFetchIdx = ModuleIdx;				break;
							}

							UParticleModule* Module = LODLevel->GetModuleAtIndex(ModuleFetchIdx);
							if (Module != NULL)
							{
								// On pass 1, clear the LODValidity flags
								// On pass 2, set it
								if (Pass == 0)
								{
									Module->LODValidity = 0;
								}
								else
								{
									Module->LODValidity |= (1 << LODIdx);
								}
							}
						}
					}
				}
			}
		}
	}
}

void UParticleSystem::SetDelay(float InDelay)
{
	Delay = InDelay;
}

#if WITH_EDITOR

bool UParticleSystem::RemoveAllDuplicateModules(bool bInMarkForCooker, TMap<UObject*,bool>* OutRemovedModules)
{
	// Generate a map of module classes used to instances of those modules 
	TMap<UClass*,TMap<UParticleModule*,int32> > ClassToModulesMap;
	for (int32 EmitterIdx = 0; EmitterIdx < Emitters.Num(); EmitterIdx++)
	{
		UParticleEmitter* Emitter = Emitters[EmitterIdx];
		if (Emitter != NULL)
		{
			if (Emitter->bCookedOut == false)
			{
				for (int32 LODIdx = 0; LODIdx < Emitter->LODLevels.Num(); LODIdx++)
				{
					UParticleLODLevel* LODLevel = Emitter->LODLevels[LODIdx];
					if (LODLevel != NULL)
					{
						for (int32 ModuleIdx = -1; ModuleIdx < LODLevel->Modules.Num(); ModuleIdx++)
						{
							UParticleModule* Module = NULL;
							if (ModuleIdx == -1)
							{
								Module = LODLevel->SpawnModule;
							}
							else
							{
								Module = LODLevel->Modules[ModuleIdx];
							}
							if (Module != NULL)
							{
								TMap<UParticleModule*,int32>* ModuleList = ClassToModulesMap.Find(Module->GetClass());
								if (ModuleList == NULL)
								{
									TMap<UParticleModule*,int32> TempModuleList;
									ClassToModulesMap.Add(Module->GetClass(), TempModuleList);
									ModuleList = ClassToModulesMap.Find(Module->GetClass());
								}
								check(ModuleList);
								int32* ModuleCount = ModuleList->Find(Module);
								if (ModuleCount == NULL)
								{
									int32 TempModuleCount = 0;
									ModuleList->Add(Module, TempModuleCount);
									ModuleCount = ModuleList->Find(Module);
								}
								check(ModuleCount);
								(*ModuleCount)++;
							}
						}
					}
				}
			}
		}
	}

	// Now we have a list of module classes and the modules they contain...
	// Find modules of the same class that have the exact same settings.
	TMap<UParticleModule*, TArray<UParticleModule*> > DuplicateModules;
	TMap<UParticleModule*,bool> FoundAsADupeModules;
	TMap<UParticleModule*, UParticleModule*> ReplaceModuleMap;
	bool bRemoveDuplicates = true;
	for (TMap<UClass*,TMap<UParticleModule*,int32> >::TIterator ModClassIt(ClassToModulesMap); ModClassIt; ++ModClassIt)
	{
		UClass* ModuleClass = ModClassIt.Key();
		TMap<UParticleModule*,int32>& ModuleMap = ModClassIt.Value();
		if (ModuleMap.Num() > 1)
		{
			// There is more than one of this module, so see if there are dupes...
			TArray<UParticleModule*> ModuleArray;
			for (TMap<UParticleModule*,int32>::TIterator ModuleIt(ModuleMap); ModuleIt; ++ModuleIt)
			{
				ModuleArray.Add(ModuleIt.Key());
			}

			// For each module, see if it it a duplicate of another
			for (int32 ModuleIdx = 0; ModuleIdx < ModuleArray.Num(); ModuleIdx++)
			{
				UParticleModule* SourceModule = ModuleArray[ModuleIdx];
				if (FoundAsADupeModules.Find(SourceModule) == NULL)
				{
					for (int32 InnerModuleIdx = ModuleIdx + 1; InnerModuleIdx < ModuleArray.Num(); InnerModuleIdx++)
					{
						UParticleModule* CheckModule = ModuleArray[InnerModuleIdx];
						if (FoundAsADupeModules.Find(CheckModule) == NULL)
						{
							bool bIsDifferent = false;
							static const FName CascadeCategory("Cascade");
							// Copy non component properties from the old actor to the new actor
							for (FProperty* Property = ModuleClass->PropertyLink; Property != NULL; Property = Property->PropertyLinkNext)
							{
								bool bIsTransient = (Property->PropertyFlags & CPF_Transient) != 0;
								bool bIsEditorOnly = (Property->PropertyFlags & CPF_EditorOnly) != 0;
								bool bIsCascade = (FObjectEditorUtils::GetCategoryFName(Property) == CascadeCategory);
								// Ignore 'Cascade' category, transient, native and EditorOnly properties...
								if (!bIsTransient && !bIsEditorOnly && !bIsCascade)
								{
									for( int32 iProp=0; iProp<Property->ArrayDim; iProp++ )
									{
										bool bIsIdentical = Property->Identical_InContainer(SourceModule, CheckModule, iProp, PPF_DeepComparison);
										if (bIsIdentical == false)
										{
											bIsDifferent = true;
											break;
										}
									}
								}
							}

							if (bIsDifferent == false)
							{
								TArray<UParticleModule*>* DupedModules = DuplicateModules.Find(SourceModule);
								if (DupedModules == NULL)
								{
									TArray<UParticleModule*> TempDupedModules;
									DuplicateModules.Add(SourceModule, TempDupedModules);
									DupedModules = DuplicateModules.Find(SourceModule);
								}
								check(DupedModules);
								if (ReplaceModuleMap.Find(CheckModule) == NULL)
								{
									ReplaceModuleMap.Add(CheckModule, SourceModule);
								}
								else
								{
									UE_LOGF(LogParticles, Error, "Module already in replacement map - ABORTING CONVERSION!!!!");
									bRemoveDuplicates = false;
								}
								DupedModules->AddUnique(CheckModule);
								FoundAsADupeModules.Add(CheckModule, true);
							}
						}
					}
				}
			}
		}
	}

	// If not errors were found, and there are duplicates, remove them...
	if (bRemoveDuplicates && (ReplaceModuleMap.Num() > 0))
	{
		TArray<UParticleModule*> RemovedModules;
		for (int32 EmitterIdx = 0; EmitterIdx < Emitters.Num(); EmitterIdx++)
		{
			UParticleEmitter* Emitter = Emitters[EmitterIdx];
			if (Emitter != NULL)
			{
				if (Emitter->bCookedOut == false)
				{
					for (int32 LODIdx = 0; LODIdx < Emitter->LODLevels.Num(); LODIdx++)
					{
						UParticleLODLevel* LODLevel = Emitter->LODLevels[LODIdx];
						if (LODLevel != NULL)
						{
							for (int32 ModuleIdx = -1; ModuleIdx < LODLevel->Modules.Num(); ModuleIdx++)
							{
								UParticleModule* Module = NULL;
								if (ModuleIdx == -1)
								{
									Module = LODLevel->SpawnModule;
								}
								else
								{
									Module = LODLevel->Modules[ModuleIdx];
								}
								if (Module != NULL)
								{
									UParticleModule** ReplacementModule = ReplaceModuleMap.Find(Module);
									if (ReplacementModule != NULL)
									{
										UParticleModule* ReplaceMod = *ReplacementModule;
										if (ModuleIdx == -1)
										{
											LODLevel->SpawnModule = CastChecked<UParticleModuleSpawn>(ReplaceMod);
										}
										else
										{
											LODLevel->Modules[ModuleIdx] = ReplaceMod;
										}

										if (bInMarkForCooker == true)
										{
											RemovedModules.AddUnique(Module);
										}
									}
								}
							}
						}
					}
				}
			}
		}

		if (bInMarkForCooker == true)
		{
			for (int32 MarkIdx = 0; MarkIdx < RemovedModules.Num(); MarkIdx++)
			{
				UParticleModule* RemovedModule = RemovedModules[MarkIdx];
				RemovedModule->SetFlags(RF_Transient);
				if (OutRemovedModules != NULL)
				{
					OutRemovedModules->Add(RemovedModule, true);
				}
			}
		}

		// Update the list of modules in each emitter
		UpdateAllModuleLists();
	}

	return true;
}


void UParticleSystem::UpdateAllModuleLists()
{
	for (int32 EmitterIdx = 0; EmitterIdx < Emitters.Num(); EmitterIdx++)
	{
		UParticleEmitter* Emitter = Emitters[EmitterIdx];
		if (Emitter != NULL)
		{
			for (int32 LODIdx = 0; LODIdx < Emitter->LODLevels.Num(); LODIdx++)
			{
				UParticleLODLevel* LODLevel = Emitter->LODLevels[LODIdx];
				if (LODLevel != NULL)
				{
					LODLevel->UpdateModuleLists();
				}
			}

			// Allow type data module to cache any module info
			if(Emitter->LODLevels.Num() > 0)
			{
				UParticleLODLevel* HighLODLevel = Emitter->LODLevels[0];
				if (HighLODLevel != nullptr && HighLODLevel->TypeDataModule != nullptr)
				{
					// Allow TypeData module to cache pointers to modules
					HighLODLevel->TypeDataModule->CacheModuleInfo(Emitter);
				}
			}

			// Update any cached info from modules on the emitter
			Emitter->CacheEmitterModuleInfo();
		}
	}
}
#endif


void UParticleSystem::BuildEmitters()
{
	const int32 EmitterCount = Emitters.Num();
	for ( int32 EmitterIndex = 0; EmitterIndex < EmitterCount; ++EmitterIndex )
	{
		if (UParticleEmitter* Emitter = Emitters[EmitterIndex])
		{
			Emitter->Build();
		}
	}
}

static bool LogReasoningForAnyThreadTicking()
{
	static bool bLogThreadedParticleTicking = FParse::Param(FCommandLine::Get(), TEXT("LogThreadedParticleTicking"));
	return bLogThreadedParticleTicking;
}

void UParticleSystem::ComputeCanTickInAnyThread()
{
	check(!bIsElligibleForAsyncTickComputed);
	bIsElligibleForAsyncTickComputed = true;

	bIsElligibleForAsyncTick = true; // assume everything is async
	int32 EmitterIndex;
	for (EmitterIndex = 0; EmitterIndex < Emitters.Num(); EmitterIndex++)
	{
		UParticleEmitter* Emitter = Emitters[EmitterIndex];
		if (Emitter)
		{
			for (int32 LevelIndex = 0; LevelIndex < Emitter->LODLevels.Num(); LevelIndex++)
			{
				UParticleLODLevel* LODLevel	= Emitter->LODLevels[LevelIndex];
				if (LODLevel)
				{
					for (int32 ModuleIndex = 0; ModuleIndex < LODLevel->Modules.Num(); ModuleIndex++)
					{
						UParticleModule* Module	= LODLevel->Modules[ModuleIndex];
						if (Module && !Module->CanTickInAnyThread())
						{
							bIsElligibleForAsyncTick = false;
							if (LogReasoningForAnyThreadTicking())
							{
								UE_LOGF(LogParticles, Display, "Cannot tick %ls in parallel because module %ls in Emitter %ls cannot tick in in parallel.", *GetFullName(), *Module->GetFullName(), *Emitter->GetFullName());
							}
							else
							{
								return;
							}
						}
					}
				}
			}

		}
	}
	if (LogReasoningForAnyThreadTicking() && bIsElligibleForAsyncTick)
	{
		UE_LOGF(LogParticles, Display, "Can tick %ls in parallel.", *GetFullName());
	}
}

bool UParticleSystem::ContainsEmitterType(UClass* TypeData)
{
	for ( int32 EmitterIndex = 0; EmitterIndex < Emitters.Num(); ++EmitterIndex)
	{
		UParticleEmitter* Emitter = Emitters[EmitterIndex];
		if (Emitter)
		{
			UParticleLODLevel* LODLevel = Emitter->LODLevels[0];
			if (LODLevel && LODLevel->TypeDataModule && LODLevel->TypeDataModule->IsA(TypeData))
			{
				return true;
			}
		}
	}
	
	return false;
}

bool UParticleSystem::HasGPUEmitter() const
{
	for (int32 EmitterIndex = 0; EmitterIndex < Emitters.Num(); ++EmitterIndex)
	{
		if (Emitters[EmitterIndex] == nullptr)
		{
			continue;
		}
		// We can just check for the GPU type data at the highest LOD.
		UParticleLODLevel* LODLevel = Emitters[EmitterIndex]->LODLevels[0];
		if( LODLevel )
		{
			UParticleModule* TypeDataModule = LODLevel->TypeDataModule;
			if( TypeDataModule && TypeDataModule->IsA(UParticleModuleTypeDataGpu::StaticClass()) )
			{
				return true;
			}
		}
	}
	return false;
}
