// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	UnParticleComponent.cpp: Particle component implementation.
=============================================================================*/

#include "Particles/ParticleSystemComponent.h"
#include "Distributions/DistributionFloat.h"
#include "Distributions/DistributionFloatConstant.h"
#include "Distributions/DistributionFloatConstantCurve.h"
#include "Distributions/DistributionFloatUniform.h"
#include "Distributions/DistributionVector.h"
#include "Distributions/DistributionVectorConstant.h"
#include "Distributions/DistributionVectorConstantCurve.h"
#include "Distributions/DistributionVectorUniform.h"
#include "Engine/Engine.h"
#include "Engine/GameInstance.h"
#include "Engine/World.h"
#include "GameFramework/WorldSettings.h"
#include "Logging/MessageLog.h"
#include "Materials/MaterialInstanceDynamic.h"
#include "MeshUVChannelInfo.h"
#include "Misc/LargeWorldRenderPosition.h"
#include "Misc/MapErrors.h"
#include "Misc/UObjectToken.h"
#include "ParticleEmitterInstanceOwner.h"
#include "ParticleEmitterInstances.h"
#include "ParticleHelper.h"
#include "ParticleSystemSceneProxy.h"
#include "Particles/EmitterCameraLensEffectBase.h"
#include "Particles/FXSystemPrivate.h"
#include "Particles/ParticleEmitter.h"
#include "Particles/ParticleEventManager.h"
#include "Particles/ParticleLODLevel.h"
#include "Particles/ParticleModuleRequired.h"
#include "Particles/ParticleSystemManager.h"
#include "Particles/ParticleSystemReplay.h"
#include "Particles/TypeData/ParticleModuleTypeDataMesh.h"
#include "SceneInterface.h"
#include "StateStream/ParticleSystemStateStream.h"
#include "UObject/FrameworkObjectVersion.h"
#include "UObject/UObjectIterator.h"
#include "UnrealEngine.h"
#include "PSOPrecacheSettings.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ParticleSystemComponent)

DECLARE_CYCLE_STAT(TEXT("ParticleComponent InitParticles GT"), STAT_ParticleSystemComponent_InitParticles, STATGROUP_Particles);
DECLARE_CYCLE_STAT(TEXT("ParticleComponent SendRenderDynamicData GT"), STAT_ParticleSystemComponent_SendRenderDynamicData_Concurrent, STATGROUP_Particles);
DECLARE_CYCLE_STAT(TEXT("ParticleComponent SendRenderTransform Concurrent GT"), STAT_ParticleSystemComponent_SendRenderTransform_Concurrent, STATGROUP_Particles);
DECLARE_CYCLE_STAT(TEXT("ParticleComponent DestroyRenderState Concurrent GT"), STAT_ParticleSystemComponent_DestroyRenderState_Concurrent, STATGROUP_Particles);
DECLARE_CYCLE_STAT(TEXT("ParticleComponent CreateDynamicData GT"), STAT_ParticleSystemComponent_CreateDynamicData, STATGROUP_Particles);
DECLARE_CYCLE_STAT(TEXT("ParticleComponent CreateDynamicData Replay GT"), STAT_ParticleSystemComponent_CreateDynamicData_Replay, STATGROUP_Particles);
DECLARE_CYCLE_STAT(TEXT("ParticleComponent CreateDynamicData Capture GT"), STAT_ParticleSystemComponent_CreateDynamicData_Capture, STATGROUP_Particles);
DECLARE_CYCLE_STAT(TEXT("ParticleComponent CreateDynamicData Gather GT"), STAT_ParticleSystemComponent_CreateDynamicData_Gather, STATGROUP_Particles);
DECLARE_CYCLE_STAT(TEXT("ParticleComponent GetDynamicData GT"), STAT_ParticleSystemComponent_GetDynamicData, STATGROUP_Particles);
DECLARE_CYCLE_STAT(TEXT("ParticleComponent GetDynamicData Selected GT"), STAT_ParticleSystemComponent_GetDynamicData_Selected, STATGROUP_Particles);
DECLARE_CYCLE_STAT(TEXT("ParticleComponent CreateDynamicData GatherCapture GT"), STAT_ParticleSystemComponent_CreateDynamicData_GatherCapture, STATGROUP_Particles);
DECLARE_CYCLE_STAT(TEXT("ParticleComponent UpdateDynamicData GT"), STAT_ParticleSystemComponent_UpdateDynamicData, STATGROUP_Particles);
DECLARE_CYCLE_STAT(TEXT("ParticleComponent OrientZAxisTowardCamera GT"), STAT_UParticleSystemComponent_OrientZAxisTowardCamera, STATGROUP_Particles);
DECLARE_CYCLE_STAT(TEXT("ParticleComponent QueueFinalize GT"), STAT_UParticleSystemComponent_QueueFinalize, STATGROUP_Particles);
DECLARE_CYCLE_STAT(TEXT("ParticleComponent CheckForReset GT"), STAT_UParticleSystemComponent_CheckForReset, STATGROUP_Particles);
DECLARE_CYCLE_STAT(TEXT("ParticleComponent LOD GT"), STAT_UParticleSystemComponent_LOD, STATGROUP_Particles);
DECLARE_CYCLE_STAT(TEXT("ParticleComponent QueueTasksGT"), STAT_UParticleSystemComponent_QueueTasks, STATGROUP_Particles);
DECLARE_CYCLE_STAT(TEXT("ParticleComponent QueueAsyncGT"), STAT_UParticleSystemComponent_QueueAsync, STATGROUP_Particles);
DECLARE_CYCLE_STAT(TEXT("ParticleComponent WaitForAsyncAndFinalize GT"), STAT_UParticleSystemComponent_WaitForAsyncAndFinalize, STATGROUP_Particles);
DECLARE_CYCLE_STAT(TEXT("ParticleComponent CreateRenderState Concurrent GT"), STAT_ParticleSystemComponent_CreateRenderState_Concurrent, STATGROUP_Particles);

DECLARE_CYCLE_STAT(TEXT("PSys Comp Marshall Time GT"), STAT_UParticleSystemComponent_Marshall, STATGROUP_Particles);

CSV_DECLARE_CATEGORY_MODULE_EXTERN(CORE_API, Basic);
DEFINE_STAT(STAT_ParticlesOverview_GT);
DEFINE_STAT(STAT_ParticlesOverview_GT_CNC);
DEFINE_STAT(STAT_ParticlesOverview_RT);
DEFINE_STAT(STAT_ParticlesOverview_RT_CNC);

#include "InGamePerformanceTracker.h"

#define LOCTEXT_NAMESPACE "ParticleComponents"

DEFINE_LOG_CATEGORY(LogParticles);

const FGuid FParticleSystemCustomVersion::GUID(0x4A56EB40, 0x10F511DC, 0x92D3347E, 0xB2C96AE7);
// Register the custom version with core
FCustomVersionRegistration GRegisterParticleSystemCustomVersion(FParticleSystemCustomVersion::GUID, FParticleSystemCustomVersion::LatestVersion, TEXT("ParticleSystemVer"));

#if WITH_STATE_STREAM_ACTOR
constexpr bool UseParticleSystemStateStream = true;
#endif

//////////////////////////////////////////////////////////////////////////

int32 GParticleLODBias = 0;
FAutoConsoleVariableRef CVarParticleLODBias(
	TEXT("r.ParticleLODBias"),
	GParticleLODBias,
	TEXT("LOD bias for particle systems, default is 0"),
	ECVF_Scalability
	);

static TAutoConsoleVariable<float> CVarPruneEmittersOnCookByDetailMode(
	TEXT("fx.PruneEmittersOnCookByDetailMode"),
	0,
	TEXT("Whether to eliminate all emitters that don't match the detail mode.\n")
	TEXT("This will only work if scalability settings affecting detail mode can not be changed at runtime (depends on platform).\n"),
	ECVF_ReadOnly);

float GFXLWCTileRecache = 2;
FAutoConsoleVariableRef CVarFXLWCTileRecache(
	TEXT("fx.LWCTileRecache"),
	GFXLWCTileRecache,
	TEXT("When we cross this number of LWC tiles from where we started the FX we need to recache the LWC tile to avoid artifacts.\n")
	TEXT("When this occurs the system may need to reset, cull particles too far away, or do some additional processing to handle it.\n")
	TEXT("Setting this value to 0 will remove this behavior but could introduce rendering & simulation artifacts.\n"),
	ECVF_Default);

static bool GFXSkipZeroDeltaTime = true;
FAutoConsoleVariableRef CVarFXSkipZeroDeltaTime(
	TEXT("fx.Cascade.SkipZeroDeltaTime"),
	GFXSkipZeroDeltaTime,
	TEXT("When enabled a delta tick time of nearly 0.0 will cause us to skip the component update.\n")
	TEXT("This fixes issue like PSA_Velocity aligned sprites, but could cause issues with things that rely on accurate velocities (i.e. TSR)."),
	ECVF_Default);

/** Whether to allow particle systems to perform work. */
ENGINE_API bool GIsAllowingParticles = true;

/** Whether to calculate LOD on the GameThread in-game. */
bool GbEnableGameThreadLODCalculation = true;

namespace CascadeLocal
{
	bool			bUseTemplateDenyList = false;
	TSet<FName>		TemplateDenyList;
	FString			TemplateDenyListString;

	static void UpdateTemplateDenyList(IConsoleVariable*)
	{
		TArray<FString> Names;
		TemplateDenyListString.ParseIntoArray(Names, TEXT(","));

		TemplateDenyList.Empty();
		for (const FString& Name : Names)
		{
			TemplateDenyList.Emplace(Name);
		}

		bUseTemplateDenyList = TemplateDenyList.Num() > 0;
	}

	bool AllowTemplate(UParticleSystem* Template)
	{
		return !bUseTemplateDenyList || (Template && !TemplateDenyList.Contains(Template->GetFName()));
	}

	static FAutoConsoleVariableRef CVarCascadeSetTemplateDenyList(
		TEXT("fx.Cascade.SetTemplateDenyList"),
		TemplateDenyListString,
		TEXT("Set the template deny List to use. (i.e. P_SystemA,P_SystemB)"),
		FConsoleVariableDelegate::CreateStatic(UpdateTemplateDenyList),
		ECVF_Scalability | ECVF_Default
	);
}

// Comment this in to debug empty emitter instance templates...
//#define _PSYSCOMP_DEBUG_INVALID_EMITTER_INSTANCE_TEMPLATES_

/*-----------------------------------------------------------------------------
	Particle scene view
-----------------------------------------------------------------------------*/
FSceneView*			GParticleView = NULL;

/*-----------------------------------------------------------------------------
	Conversion functions
-----------------------------------------------------------------------------*/
void Particle_ModifyFloatDistribution(UDistributionFloat* pkDistribution, float fScale)
{
	if (pkDistribution->IsA(UDistributionFloatConstant::StaticClass()))
	{
		UDistributionFloatConstant* pkDistConstant = Cast<UDistributionFloatConstant>(pkDistribution);
		pkDistConstant->Constant *= fScale;
	}
	else
	if (pkDistribution->IsA(UDistributionFloatUniform::StaticClass()))
	{
		UDistributionFloatUniform* pkDistUniform = Cast<UDistributionFloatUniform>(pkDistribution);
		pkDistUniform->Min *= fScale;
		pkDistUniform->Max *= fScale;
	}
	else
	if (pkDistribution->IsA(UDistributionFloatConstantCurve::StaticClass()))
	{
		UDistributionFloatConstantCurve* pkDistCurve = Cast<UDistributionFloatConstantCurve>(pkDistribution);

		int32 iKeys = pkDistCurve->GetNumKeys();
		int32 iCurves = pkDistCurve->GetNumSubCurves();

		for (int32 KeyIndex = 0; KeyIndex < iKeys; KeyIndex++)
		{
			float fKeyIn = pkDistCurve->GetKeyIn(KeyIndex);
			for (int32 SubIndex = 0; SubIndex < iCurves; SubIndex++)
			{
				float fKeyOut = pkDistCurve->GetKeyOut(SubIndex, KeyIndex);
				float ArriveTangent;
				float LeaveTangent;
				pkDistCurve->GetTangents(SubIndex, KeyIndex, ArriveTangent, LeaveTangent);

				pkDistCurve->SetKeyOut(SubIndex, KeyIndex, fKeyOut * fScale);
				pkDistCurve->SetTangents(SubIndex, KeyIndex, ArriveTangent * fScale, LeaveTangent * fScale);
			}
		}
	}
}

void Particle_ModifyVectorDistribution(UDistributionVector* pkDistribution, FVector& vScale)
{
	if (pkDistribution->IsA(UDistributionVectorConstant::StaticClass()))
	{
		UDistributionVectorConstant* pkDistConstant = Cast<UDistributionVectorConstant>(pkDistribution);
		pkDistConstant->Constant *= vScale;
	}
	else
	if (pkDistribution->IsA(UDistributionVectorUniform::StaticClass()))
	{
		UDistributionVectorUniform* pkDistUniform = Cast<UDistributionVectorUniform>(pkDistribution);
		pkDistUniform->Min *= vScale;
		pkDistUniform->Max *= vScale;
	}
	else
	if (pkDistribution->IsA(UDistributionVectorConstantCurve::StaticClass()))
	{
		UDistributionVectorConstantCurve* pkDistCurve = Cast<UDistributionVectorConstantCurve>(pkDistribution);

		int32 iKeys = pkDistCurve->GetNumKeys();
		int32 iCurves = pkDistCurve->GetNumSubCurves();

		for (int32 KeyIndex = 0; KeyIndex < iKeys; KeyIndex++)
		{
			float fKeyIn = pkDistCurve->GetKeyIn(KeyIndex);
			for (int32 SubIndex = 0; SubIndex < iCurves; SubIndex++)
			{
				float fKeyOut = pkDistCurve->GetKeyOut(SubIndex, KeyIndex);
				float ArriveTangent;
				float LeaveTangent;
				pkDistCurve->GetTangents(SubIndex, KeyIndex, ArriveTangent, LeaveTangent);

				switch (SubIndex)
				{
				case 1:
					pkDistCurve->SetKeyOut(SubIndex, KeyIndex, fKeyOut * vScale.Y);
					pkDistCurve->SetTangents(SubIndex, KeyIndex, ArriveTangent * vScale.Y, LeaveTangent * vScale.Y);
					break;
				case 2:
					pkDistCurve->SetKeyOut(SubIndex, KeyIndex, fKeyOut * vScale.Z);
					pkDistCurve->SetTangents(SubIndex, KeyIndex, ArriveTangent * vScale.Z, LeaveTangent * vScale.Z);
					break;
				case 0:
				default:
					pkDistCurve->SetKeyOut(SubIndex, KeyIndex, fKeyOut * vScale.X);
					pkDistCurve->SetTangents(SubIndex, KeyIndex, ArriveTangent * vScale.X, LeaveTangent * vScale.X);
					break;
				}
			}
		}
	}
}

/** Console command to reset all particle components. */
static void ResetAllParticleComponents()
{
	for (TObjectIterator<UParticleSystemComponent> It; It; ++It)
	{
		UParticleSystemComponent* ParticleSystemComponent = *It;
		ParticleSystemComponent->ResetParticles();
		ParticleSystemComponent->ActivateSystem(true);
		ParticleSystemComponent->bIsViewRelevanceDirty = true;
		ParticleSystemComponent->CachedViewRelevanceFlags.Empty();
		ParticleSystemComponent->ConditionalCacheViewRelevanceFlags();
		ParticleSystemComponent->ReregisterComponent();
	}
}
static FAutoConsoleCommand GResetAllParticleComponentsCmd(
	TEXT("FX.RestartAll"),
	TEXT("Restarts all particle system components"),
	FConsoleCommandDelegate::CreateStatic(ResetAllParticleComponents)
	);


UFXSystemComponent::UFXSystemComponent(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{}

bool UFXSystemComponent::RequiresLWCTileRecache(const FVector3f CurrentTile, const FVector CurrentLocation)
{
	bool bNeedsRecache = false;
	const float TileRecache = GFXLWCTileRecache;
	if (TileRecache > 0.0f)
	{
		const FVector3f ActorTile = FLargeWorldRenderScalar::GetTileFor(CurrentLocation);
		const float MaxMovement = (CurrentTile - ActorTile).GetAbs().GetMax();
		bNeedsRecache = MaxMovement >= TileRecache;
	}
	return bNeedsRecache;
}

void UFXSystemComponent::PrecacheAssetPSOs(UFXSystemAsset* FXSystemAsset)
{
#if UE_WITH_PSO_PRECACHING
	if (!FApp::CanEverRender() || (!IsComponentPSOPrecachingEnabled()) || FXSystemAsset == nullptr || bPSOPrecacheCalled)
	{
		return;
	}
	bPSOPrecacheCalled = true;
	FXSystemAsset->PrecacheAssetPSOs();
	
	FGraphEventRef GraphEvent = FXSystemAsset->GetPrecachePSOsEvent();

	check(IsInGameThread() || IsInParallelGameThread());

	PSOPrecacheComponentData.Reset();

	// The asset will keep the Precache events alive, but these might be over. Avoid delaying scene proxy creation if everything is finished
	bool bAllEventsDone = GraphEvent == nullptr || GraphEvent->IsComplete();

	TArray<FMaterialPSOPrecacheRequestID> MaterialPSOPrecacheRequestIDs;
	FGraphEventArray PSOPrecacheCompileEvents;
	if (!bAllEventsDone)
	{
		MaterialPSOPrecacheRequestIDs.Append(FXSystemAsset->GetMaterialPSOPrecacheRequestIDs());
		PSOPrecacheCompileEvents.Add(GraphEvent);
	}

	SetMaterialPSOPrecacheData(MaterialPSOPrecacheRequestIDs, PSOPrecacheCompileEvents);
#endif // UE_WITH_PSO_PRECACHING
}


class UParticleSystemComponent::FInstanceOwner : public IParticleEmitterInstanceOwner
{
public:
	FInstanceOwner(UParticleSystemComponent* InComponent) : Component(InComponent) {}

	virtual const FTransform& GetAsyncComponentToWorld() const override { return Component->GetAsyncComponentToWorld(); }

	virtual UObject* GetDistributionData() const override { return Component; }
	virtual const FTransform& GetComponentTransform() const override { return Component->GetComponentTransform(); }
	virtual FRotator GetComponentRotation() const override { return Component->GetComponentRotation(); }
	virtual const FTransform& GetComponentToWorld() const override { return Component->GetComponentToWorld(); }
	virtual const FBoxSphereBounds& GetBounds() const override { return Component->Bounds; }
	virtual TWeakObjectPtr<UWorld> GetWeakWorld() const override { return Component->GetWorld(); }
	virtual bool HasWorld() const override { return Component->GetWorld() != nullptr; }
	virtual bool HasWorldSettings() const override { UWorld* World = Component->GetWorld(); return World ? World->GetWorldSettings() != nullptr : false; };
	virtual bool IsGameWorld() const { UWorld* World = Component->GetWorld(); return World ? World->IsGameWorld() : false; }
	virtual float GetWorldTimeSeconds() const override { UWorld* World = Component->GetWorld(); return World ? World->GetTimeSeconds() : 0.0f; }
	virtual float GetWorldEffectiveTimeDilation() const override
	{
		if (UWorld* World = Component->GetWorld())
		{
			AWorldSettings* Settings = World->GetWorldSettings();
			return Settings ? Settings->GetEffectiveTimeDilation() : 1.0f;
		}
		return 1.0f;
	}
	virtual FIntVector GetWorldOriginLocation() const override
	{
		if (UWorld* World = Component->GetWorld())
		{
			return World->OriginLocation;
		}
		return FIntVector(ForceInitToZero);
	}
	virtual FSceneInterface* GetScene() const override
	{
		if (UWorld* World = Component->GetWorld())
		{
			return World->Scene;
		}
		return nullptr;
	}

	virtual bool GetFloatParameter(const FName InName, float& OutFloat) override { return Component->GetFloatParameter(InName, OutFloat); }
	virtual const FVector3f& GetLWCTile() const override { return Component->GetLWCTile(); }
	virtual FString GetName() const override { return Component->GetName(); }
	virtual FString GetFullName() const override { return Component->GetFullName(); }
	virtual FString GetPathName() const override { return Component->GetPathName(); }
	virtual bool IsActive() const override { return Component->IsActive(); }
	virtual bool IsValidLowLevel() const override { return Component->IsValidLowLevel(); }
	virtual TArrayView<const FParticleSysParam> GetAsyncInstanceParameters() override { return Component->GetAsyncInstanceParameters(); }
	virtual int32 GetCurrentDetailMode() const override { return Component->GetCurrentDetailMode(); }
	virtual int32 GetCurrentLODIndex() const override { return Component->GetCurrentLODIndex(); }
	virtual const FVector& GetPartSysVelocity() const override { return Component->PartSysVelocity; }
	virtual const FVector& GetOldPosition() const override { return Component->OldPosition; }
	virtual FFXSystem* GetFXSystem() const override { return Component->FXSystem; }
	virtual const UParticleSystem* GetTemplate() const override { return Component->Template; }
	virtual TArrayView<const FParticleSysParam> GetInstanceParameters() const override { return Component->InstanceParameters; }
	virtual TArrayView<FParticleEmitterInstance*> GetEmitterInstances() const override { return Component->EmitterInstances; }
	virtual TArrayView<TObjectPtr<UMaterialInterface>> GetEmitterMaterials() const override { return Component->EmitterMaterials; }
	virtual FPrimitiveSceneProxy* GetSceneProxy() const override { return Component->SceneProxy; }
	virtual bool GetIsWarmingUp() const override { return Component->bWarmingUp; }
	virtual bool GetJustRegistered() const override { return Component->bJustRegistered; }
	virtual float GetWarmupTime() const override { return Component->WarmupTime; }
	virtual float GetEmitterDelay() const override { return Component->EmitterDelay; }
	virtual FRandomStream& GetRandomStream() override { return Component->RandomStream; }

	virtual void SetComponentToWorld(const FTransform& NewComponentToWorld) override { Component->SetComponentToWorld(NewComponentToWorld); }
	virtual void DeactivateNextTick() override { Component->DeactivaateNextTick(); }

	virtual UParticleSystemComponent* AsComponent() const override { return Component; }

	virtual void ReportEventSpawn(const FName InEventName, const float InEmitterTime, const FVector InLocation, const FVector InVelocity, const TArray<class UParticleModuleEventSendToGame*>& InEventData) override
	{
		Component->ReportEventSpawn(InEventName, InEmitterTime, InLocation, InVelocity, InEventData);
	}

	virtual void ReportEventDeath(const FName InEventName, const float InEmitterTime, const FVector InLocation, const FVector InVelocity, const TArray<class UParticleModuleEventSendToGame*>& InEventData, const float InParticleTime) override
	{
		Component->ReportEventDeath(InEventName, InEmitterTime, InLocation, InVelocity, InEventData, InParticleTime);
	}

	virtual void ReportEventCollision(const FName InEventName, const float InEmitterTime, const FVector InLocation, const FVector InDirection, const FVector InVelocity, const TArray<class UParticleModuleEventSendToGame*>& InEventData, const float InParticleTime, const FVector InNormal, const float InTime, const int32 InItem, const FName InBoneName, UPhysicalMaterial* PhysMat) override
	{
		Component->ReportEventCollision(InEventName, InEmitterTime, InLocation, InDirection, InVelocity, InEventData, InParticleTime, InNormal, InTime, InItem, InBoneName, PhysMat);
	}

	virtual void ReportEventBurst(const FName InEventName, const float InEmitterTime, const int32 ParticleCount, const FVector InLocation, const TArray<class UParticleModuleEventSendToGame*>& InEventData) override
	{
		Component->ReportEventBurst(InEventName, InEmitterTime, ParticleCount, InLocation, InEventData);
	}

	virtual TArrayView<FParticleEventSpawnData> GetSpawnEvents() const override { return Component->SpawnEvents; }
	virtual TArrayView<FParticleEventDeathData> GetDeathEvents() const override { return Component->DeathEvents; }
	virtual TArrayView<FParticleEventCollideData> GetCollisionEvents() const override { return Component->CollisionEvents; }
	virtual TArrayView<FParticleEventBurstData> GetBurstEvents() const override { return Component->BurstEvents; }
	virtual TArrayView<FParticleEventKismetData> GetKismetEvents() const override { return Component->KismetEvents; }

	UParticleSystemComponent* Component;
};

FOnSystemPreActivationChange UParticleSystemComponent::OnSystemPreActivationChange;

UParticleSystemComponent::UParticleSystemComponent(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer), FXSystem(NULL), ReleaseResourcesFence(NULL)
{
	PrimaryComponentTick.bCanEverTick = true;
	PrimaryComponentTick.TickGroup = TG_DuringPhysics;
	PrimaryComponentTick.bAllowTickOnDedicatedServer = false;
	bTickInEditor = true;
	MaxTimeBeforeForceUpdateTransform = 5.0f;
	bAutoActivate = true;
	bResetOnDetach = false;
	bOldPositionValid = false;
	OldPosition = FVector::ZeroVector;

	RandomStream.Initialize(FApp::bUseFixedSeed ? GetFName() : NAME_None);

	PartSysVelocity = FVector::ZeroVector;

	WarmupTime = 0.0f;
	SecondsBeforeInactive = 1.0f;
	bIsTransformDirty = false;
	bSkipUpdateDynamicDataDuringTick = false;
	bIsViewRelevanceDirty = true;
	CustomTimeDilation = 1.0f;
	bAllowConcurrentTick = true;
	bAsyncWorkOutstanding = false;
	PoolingMethod = EPSCPoolMethod::None;
	bWasActive = false;
#if WITH_EDITORONLY_DATA
	EditorDetailMode = -1;
#endif // WITH_EDITORONLY_DATA
	SetCollisionProfileName(UCollisionProfile::NoCollision_ProfileName);
	SetGenerateOverlapEvents(false);

	bCastVolumetricTranslucentShadow = true;

	// Disable receiving decals by default.
	bReceivesDecals = false;

	// Don't need to call OnUpdateTransform, no physics state to update
	bWantsOnUpdateTransform = false;

	SavedAutoAttachRelativeScale3D = FVector(1.f, 1.f, 1.f);
	TimeSinceLastTick = 0;

	RequiredSignificance = EParticleSignificanceLevel::Low;
	LastSignificantTime = 0.0f;
	bIsManagingSignificance = 0;
	bWasManagingSignificance = 0;
	bIsDuringRegister = 0;

	ManagerHandle = INDEX_NONE;
	bPendingManagerAdd = false;
	bPendingManagerRemove = false;

	bExcludeFromLightAttachmentGroup = true;
}

void UParticleSystemComponent::SetRequiredSignificance(EParticleSignificanceLevel NewRequiredSignificance)
{
	if (Template)
	{
		RequiredSignificance = NewRequiredSignificance;

		EParticleSystemInsignificanceReaction Reaction = Template->InsignificantReaction;
		if (Template->InsignificantReaction == EParticleSystemInsignificanceReaction::Auto)
		{
			Reaction = Template->IsLooping() ? EParticleSystemInsignificanceReaction::DisableTick : EParticleSystemInsignificanceReaction::Complete;
		}

		//If our tick is disabled we need to work out if we should re-enable it based on this new significance
		if (!IsComponentTickEnabled() && Reaction == EParticleSystemInsignificanceReaction::DisableTick && Template->GetHighestSignificance() >= NewRequiredSignificance)
		{
			//Set us to be significant again.
			OnSignificanceChanged(true, true, true);
		}
	}
}

void UParticleSystemComponent::OnSignificanceChanged(bool bSignificant, bool bApplyToEmitters, bool bAsync)
{
	ForceAsyncWorkCompletion(STALL, false);
	int32 LocalNumSignificantEmitters = 0;
	bool bTickIsEnabled = IsComponentTickEnabled();
	bool bNewTickEnabled = bTickIsEnabled;
	if (bSignificant)
	{
		bNewTickEnabled = true;

		if (bApplyToEmitters && EmitterInstances.Num() > 0)
		{
			//Mark any emitters as significant if needed.
			for (FParticleEmitterInstance* Inst : EmitterInstances)
			{
				if (Inst)
				{
					if (Inst->SpriteTemplate->IsSignificant(RequiredSignificance))
					{
						Inst->bEnabled = true;
						Inst->SetHaltSpawning(false);
						Inst->SetFakeBurstWhenSpawningSupressed(false);
						++LocalNumSignificantEmitters;
					}
				}
				else
				{
					++LocalNumSignificantEmitters;//Set significant for missing emitters due to other reasons such as detail mode.
				}
			}

			if (LocalNumSignificantEmitters == 0)
			{
				UE_LOGF(LogParticles, Warning, "Setting PSC as significant but it has no significant emitters. %ls Template: %ls", *GetFullName(), *Template->GetFullName());
			}
			NumSignificantEmitters = LocalNumSignificantEmitters;
		}
	}
	else
	{
		bNewTickEnabled = false;

		if (bApplyToEmitters && EmitterInstances.Num() > 0)
		{
			//Mark any emitters as significant if needed.
			for (FParticleEmitterInstance* Inst : EmitterInstances)
			{
				if (Inst)
				{
					UParticleLODLevel* SpriteLODLevel = Inst->SpriteTemplate->GetCurrentLODLevel(Inst);
					if (SpriteLODLevel && SpriteLODLevel->bEnabled)//Checking these too as they can stop us from marking emitters as signficant during update and trigger setting insignificant.
					{
						if (Inst->SpriteTemplate->IsSignificant(RequiredSignificance))
						{
							++LocalNumSignificantEmitters;
						}
						else
						{
							Inst->bEnabled = false;
							Inst->SetHaltSpawning(true);
							Inst->SetFakeBurstWhenSpawningSupressed(true);
						}
					}
				}
			}

			if (LocalNumSignificantEmitters > 0)
			{
				UE_LOGF(LogParticles, Warning, "Setting PSC as not significant but it has some significant emitters. %ls Template: %ls", *GetFullName(), *Template->GetFullName());
			}

			NumSignificantEmitters = LocalNumSignificantEmitters;
		}

		EParticleSystemInsignificanceReaction Reaction = Template->InsignificantReaction;
		if (Template->InsignificantReaction == EParticleSystemInsignificanceReaction::Auto)
		{
			Reaction = Template->IsLooping() ? EParticleSystemInsignificanceReaction::DisableTick : EParticleSystemInsignificanceReaction::Complete;
		}

		switch (Reaction)
		{
			case EParticleSystemInsignificanceReaction::Complete:
			{
				Complete();
			}
				break;
			case EParticleSystemInsignificanceReaction::DisableTick:
			{
				bNewTickEnabled = false;
			}
				break;
			case EParticleSystemInsignificanceReaction::DisableTickAndKill:
			{
				KillParticlesForced();//TODO: Make this actually free memory.
				bNewTickEnabled = false;
			}
				break;
		}
	}

	//If we've been deactivated then we have to be ticking so that the system can complete correctly.
	bNewTickEnabled |= bWasDeactivated;

	if(bTickIsEnabled != bNewTickEnabled)
	{
		if (bAsync)
		{
			SetComponentTickEnabledAsync(bNewTickEnabled);
		}
		else
		{
			SetComponentTickEnabled(bNewTickEnabled);
		}
	}
}

bool UParticleSystemComponent::ShouldManageSignificance() const
{
	return Template ? Template->ShouldManageSignificance() : false;
}

float UParticleSystemComponent::GetApproxDistanceSquared(FVector Point) const
{
	return Bounds.ComputeSquaredDistanceFromBoxToPoint(Point);
	//TODO: Consider beam line segment?
}

bool UParticleSystemComponent::CanBeOccluded()const
{
	return Template->OcclusionBoundsMethod != EPSOBM_None && 
		(Template->FixedRelativeBoundingBox.IsValid || (Template->OcclusionBoundsMethod == EPSOBM_CustomBounds)); //We can only be occluded if we have fixed bounds or custom occlusion bounds.
}

bool UParticleSystemComponent::CanSkipTickDueToVisibility()
{
	if (Template && Template->IsLooping() && CanConsiderInvisible() && !bWasDeactivated)
	{
		bForcedInActive = true;
		SpawnEvents.Empty();
		DeathEvents.Empty();
		CollisionEvents.Empty();
		KismetEvents.Empty();

		if (bIsManagingSignificance && Template->GetHighestSignificance() < RequiredSignificance)
		{
			//We're definitely insignificant so we can stop ticking entirely.
			OnSignificanceChanged(false, true);
		}

		return true;
	}

	return false;
}

bool UParticleSystemComponent::CanConsiderInvisible()const
{
	UWorld* World = GetWorld();
	if (World && Template)
	{
		const float MaxSecondsBeforeInactive = FMath::Max(SecondsBeforeInactive, Template->SecondsBeforeInactive);

		// Clamp MaxSecondsBeforeInactive to be at least twice the maximum smoothed frame time (45.45ms) because the rendering thread runs one 
		// frame behind the game thread and so smaller time differences cannot be reliably detected.
		const float ClampedMaxSecondsBeforeInactive = MaxSecondsBeforeInactive > 0 ? FMath::Max(MaxSecondsBeforeInactive, 0.1f) : 0.0f;
		if (ClampedMaxSecondsBeforeInactive > 0.0f && AccumTickTime > ClampedMaxSecondsBeforeInactive && World->IsGameWorld())
		{
			return (GetLastRenderTime() > 0.0f) && (World->GetTimeSeconds() > (GetLastRenderTime() + ClampedMaxSecondsBeforeInactive));
		}
	}
	return false;
}

void DetailModeSink()
{
	//This Cvar sink can happen before the one which primes the cached scalability cvars so we must grab this ourselves.
	IConsoleManager& ConsoleMan = IConsoleManager::Get();
	static const auto DetailMode = ConsoleMan.FindTConsoleVariableDataInt(TEXT("r.DetailMode"));
	int32 NewDetailMode = DetailMode->GetValueOnGameThread();
	static int32 CachedDetailMode = NewDetailMode;

	if (CachedDetailMode != NewDetailMode)
	{
		CachedDetailMode = NewDetailMode;

		for (TObjectIterator<UParticleSystemComponent> It; It; ++It)
		{
			//We must also reset on next tick rather than immediately as the cached cvar values are read internally to determin detail mode.
			It->ResetNextTick();
		}
	}
}

static FAutoConsoleVariableSink CVarDetailModeSink(FConsoleCommandDelegate::CreateStatic(&DetailModeSink));

void UParticleSystemComponent::ForceReset()
{
#if WITH_EDITOR
	//If we're resetting in the editor, cached emitter values may now be invalid.
	if (Template != nullptr)
	{
		Template->UpdateAllModuleLists();
	}
#endif

	bool bOldActive = IsActive();
	ResetParticles(true);
	if (bOldActive)
	{
		ActivateSystem();
	}
	else
	{
		InitializeSystem();
	}
}

void UParticleSystemComponent::MarshalParamsForAsyncTick()
{
	SCOPE_CYCLE_COUNTER(STAT_UParticleSystemComponent_Marshall);
	bAsyncDataCopyIsValid = true;
	check(!bParallelRenderThreadUpdate);
	AsyncComponentToWorld = GetComponentTransform();
	AsyncInstanceParameters.Reset();
	AsyncInstanceParameters.Append(InstanceParameters);
	AsyncBounds = Bounds;
	AsyncPartSysVelocity = PartSysVelocity;

	//cache component to world of each actor that trails may use
	for (FParticleSysParam& ParticleSysParam : AsyncInstanceParameters)
	{
		ParticleSysParam.UpdateAsyncActorCache();
	}

	bAsyncWorkOutstanding = true;
}

#if WITH_EDITOR
void UParticleSystemComponent::CheckForErrors()
{
	check(IsInGameThread());
	ForceAsyncWorkCompletion(ENSURE_AND_STALL);
	for (int32 IPIndex = 0; IPIndex < InstanceParameters.Num(); IPIndex++)
	{
		FParticleSysParam& Param = InstanceParameters[IPIndex];
		if (Param.ParamType == PSPT_Actor)
		{
			if (Param.Actor == NULL)
			{
				FFormatNamedArguments Arguments;
				Arguments.Add(TEXT("InstanceParamIndex"), IPIndex);
				Arguments.Add(TEXT("PathName"), FText::FromString(GetPathName()));
				FMessageLog("MapCheck").Warning()
					->AddToken(FUObjectToken::Create(this))
					->AddToken(FTextToken::Create(FText::Format( LOCTEXT( "MapCheck_Message_PSysCompErrorEmptyActorRef", "PSysComp has an empty parameter actor reference at index {InstanceParamIndex} ({PathName})" ), Arguments ) ))
					->AddToken(FMapErrorToken::Create(FMapErrors::PSysCompErrorEmptyActorRef));
			}
		}
		else
		if (Param.ParamType == PSPT_Material)
		{
			if (Param.Material == NULL)
			{
				FFormatNamedArguments Arguments;
				Arguments.Add(TEXT("InstanceParamIndex"), IPIndex);
				Arguments.Add(TEXT("PathName"), FText::FromString(GetPathName()));
				FMessageLog("MapCheck").Warning()
					->AddToken(FUObjectToken::Create(this))
					->AddToken(FTextToken::Create(FText::Format( LOCTEXT( "MapCheck_Message_PSysCompErrorEmptyMaterialRef", "PSysComp has an empty parameter material reference at index {InstanceParamIndex} ({PathName})" ), Arguments ) ))
					->AddToken(FMapErrorToken::Create(FMapErrors::PSysCompErrorEmptyMaterialRef));
			}
		}
	}

	static const auto CVar = IConsoleManager::Get().FindConsoleVariable(TEXT("r.Shadow.TranslucentPerObject.ProjectEnabled"));
	if (bCastVolumetricTranslucentShadow && CastShadow && bCastDynamicShadow && CVar && CVar->GetInt() == 0)
	{
		FMessageLog("MapCheck").Warning()
			->AddToken(FUObjectToken::Create(this))
			->AddToken(FTextToken::Create(LOCTEXT("MapCheck_Message_NoTranslucentShadowSupport", "Component is a using CastVolumetricTranslucentShadow but this feature is disabled for the project! Turn on r.Shadow.TranslucentPerObject.ProjectEnabled in a project ini if required.")))
			->AddToken(FMapErrorToken::Create(FMapErrors::PrimitiveComponentHasInvalidTranslucentShadowSetting));
	}
}
#endif

void UParticleSystemComponent::PostLoad()
{
	ForceAsyncWorkCompletion(ENSURE_AND_STALL);
	Super::PostLoad();

	if (Template)
	{
		Template->ConditionalPostLoad();
	}
	bIsViewRelevanceDirty = true;

	if (ShouldBeTickManaged())
	{
		PrimaryComponentTick.bStartWithTickEnabled = false;
	}

	if (Template && UPSOPrecacheSettingsManager::GetSettings().ParticlePrecachingTime > 1)
	{
		Template->ConditionalPostLoad();
		Template->PrecacheAssetPSOs();
	}
}

void UParticleSystemComponent::Serialize( FArchive& Ar )
{
	ForceAsyncWorkCompletion(ENSURE_AND_STALL);
	Super::Serialize( Ar );
	
	// Take instance particle count/ size into account.
	for (int32 InstanceIndex = 0; InstanceIndex < EmitterInstances.Num(); InstanceIndex++)
	{
		FParticleEmitterInstance* EmitterInstance = EmitterInstances[InstanceIndex];
		if( EmitterInstance != NULL )
		{
			int32 Num, Max;
			EmitterInstance->GetAllocatedSize(Num, Max);
			Ar.CountBytes(Num, Max);
		}
	}

	Ar.UsingCustomVersion(FFrameworkObjectVersion::GUID);
#if WITH_EDITORONLY_DATA

	if (Ar.CustomVer(FFrameworkObjectVersion::GUID) < FFrameworkObjectVersion::ExplicitAttachmentRules)
	{
		USceneComponent::ConvertAttachLocation(AutoAttachLocationType_DEPRECATED, AutoAttachLocationRule, AutoAttachRotationRule, AutoAttachScaleRule);
	}
#endif
}

void UParticleSystemComponent::BeginDestroy()
{
	ForceAsyncWorkCompletion(ENSURE_AND_STALL, true, true);
	Super::BeginDestroy();

	if (PoolingMethod == EPSCPoolMethod::AutoRelease || PoolingMethod == EPSCPoolMethod::ManualRelease)
	{
		UE_LOGF(LogParticles, Warning, "Pooled Particle System Component is being destroyed! Do not manually destoy PSCs that are being pooled.\n           ParticleSystem=%ls\n           Template:%ls",
			*GetPathName(), Template ? *Template->GetPathName() : TEXT("NULL"));
	}
	else if (PoolingMethod == EPSCPoolMethod::FreeInPool)
	{
		UE_LOGF(LogParticles, Warning, "Pooled Particle System Component that has already been released to the pool is being destroyed!\nWe should not even be keeping references to these components after they have been released to the pool!\n           ParticleSystem=%ls\n           Template:%ls",
			*GetPathName(), Template ? *Template->GetPathName() : TEXT("NULL"));
	}

	// Call delegate to ensure we unregister from Significance Manager regardless if this PSC is active or not
	OnSystemPreActivationChange.Broadcast(this, false);
	ResetParticles(true);
}

void UParticleSystemComponent::FinishDestroy()
{
	ForceAsyncWorkCompletion(ENSURE_AND_STALL, true, true);
	for (int32 EmitterIndex = 0; EmitterIndex < EmitterInstances.Num(); EmitterIndex++)
	{
		FParticleEmitterInstance* EmitInst = EmitterInstances[EmitterIndex];
		if (EmitInst)
		{
#if STATS
			EmitInst->PreDestructorCall();
#endif
			delete EmitInst;
			EmitterInstances[EmitterIndex] = NULL;
		}
	}
	Super::FinishDestroy();
}

void UParticleSystemComponent::GetResourceSizeEx(FResourceSizeEx& CumulativeResourceSize)
{
	ForceAsyncWorkCompletion(ENSURE_AND_STALL);

	Super::GetResourceSizeEx(CumulativeResourceSize);
	for (int32 EmitterIdx = 0; EmitterIdx < EmitterInstances.Num(); EmitterIdx++)
	{
		FParticleEmitterInstance* EmitterInstance = EmitterInstances[EmitterIdx];
		if (EmitterInstance != NULL)
		{
			// If the data manager has the PSys, force it to report, regardless of a PSysComp scene info being present...
			EmitterInstance->GetResourceSizeEx(CumulativeResourceSize);
		}
	}
}


bool UParticleSystemComponent::ParticleLineCheck(FHitResult& Hit, AActor* SourceActor, const FVector& End, const FVector& Start, const FVector& HalfExtent, const FCollisionObjectQueryParams& ObjectParams)
{
	check(GetWorld());
	if ( HalfExtent.IsZero() )
	{
		FCollisionQueryParams QueryParams(SCENE_QUERY_STAT(ParticleCollision), true, SourceActor);
		QueryParams.bReturnPhysicalMaterial = true;
		return GetWorld()->LineTraceSingleByObjectType(Hit, Start, End, ObjectParams, QueryParams);
	}
	else
	{
		FCollisionQueryParams BoxParams(SCENE_QUERY_STAT(ParticleCollision));
		BoxParams.AddIgnoredActor(SourceActor);
		BoxParams.bReturnPhysicalMaterial = true;
		return GetWorld()->SweepSingleByObjectType(Hit, Start, End, FQuat::Identity, ObjectParams, FCollisionShape::MakeBox(HalfExtent), BoxParams);
	}
}

void UParticleSystemComponent::OnRegister()
{
	FGuardValue_Bitfield(bIsDuringRegister, true);
	
	ForceAsyncWorkCompletion(STALL);
	check(FXSystem == nullptr);

	UWorld* World = GetWorld();
	check(World != nullptr);

	if (World->Scene)
	{
		FFXSystemInterface*  FXSystemInterface = World->Scene->GetFXSystem();
		if (FXSystemInterface)
		{
			FXSystem = static_cast<FFXSystem*>(FXSystemInterface->GetInterface(FFXSystem::Name));
		}
	}

	if (bAutoManageAttachment && !IsActive())
	{
		// Detach from current parent, we are supposed to wait for activation.
		if (GetAttachParent())
		{
			// If no auto attach parent override, use the current parent when we activate
			if (!AutoAttachParent.IsValid())
			{
				AutoAttachParent = GetAttachParent();
			}
			// If no auto attach socket override, use current socket when we activate
			if (AutoAttachSocketName == NAME_None)
			{
				AutoAttachSocketName = GetAttachSocketName();
			}

			// If in a game world, detach now if necessary. Activation will cause auto-attachment.
			if (World->IsGameWorld())
			{
				// Prevent attachment before Super::OnRegister() tries to attach us, since we only attach when activated.
				if (GetAttachParent()->GetAttachChildren().Contains(this))
				{
					// Only detach if we are not about to auto attach to the same target, that would be wasteful.
					if (!bAutoActivate || (AutoAttachLocationRule != EAttachmentRule::KeepRelative && AutoAttachRotationRule != EAttachmentRule::KeepRelative && AutoAttachScaleRule != EAttachmentRule::KeepRelative) || (AutoAttachSocketName != GetAttachSocketName()) || (AutoAttachParent != GetAttachParent()))
					{
						DetachFromComponent(FDetachmentTransformRules(EDetachmentRule::KeepRelative, /*bCallModify=*/ false));
					}
				}
				else
				{
					SetupAttachment(nullptr, NAME_None);
				}
			}
		}

		SavedAutoAttachRelativeLocation = GetRelativeLocation();
		SavedAutoAttachRelativeRotation = GetRelativeRotation();
		SavedAutoAttachRelativeScale3D = GetRelativeScale3D();
	}

	if (ShouldBeTickManaged())
	{
		PrimaryComponentTick.bStartWithTickEnabled = false;
	}

	Super::OnRegister();

	// If we were active before but are not now, activate us
	if (bWasActive && !IsActive())
	{
		Activate(true);
	}

	UE_LOGF(LogParticles,Verbose,
		"OnRegister %ls Component=0x%p World=0x%p Scene=0x%p FXSystem=0x%p",
		Template != NULL ? *Template->GetName() : TEXT("NULL"), this, GetWorld(), World->Scene, FXSystem);

	if (LODLevel == -1)
	{
		// Force it to LODLevel 0
		LODLevel = 0;
	}

	// Deal with the case where the particle component is attached to an actor in a hidden sublevel. Without this, the component will be visible instead of being hidden as well.
	if (CachedLevelCollection == nullptr && GetOwner() == nullptr && IsValid(GetAttachParent()))
	{
		const ULevel* const AttachParentLevel = GetAttachParent()->GetComponentLevel();
		CachedLevelCollection = AttachParentLevel ? AttachParentLevel->GetCachedLevelCollection() : nullptr;
	}
}

void UParticleSystemComponent::OnUnregister()
{
	ForceAsyncWorkCompletion(STALL);
	UE_LOGF(LogParticles,Verbose,
		"OnUnregister %ls Component=0x%p Scene=0x%p FXSystem=0x%p",
		Template != NULL ? *Template->GetName() : TEXT("NULL"), this, GetWorld()->Scene, FXSystem);

	bWasActive = IsActive() && !bWasDeactivated;

	check(GetWorld());
	SetComponentTickEnabled(false);

	bool bEmptyInstances = !bAllowRecycling || GetWorld()->bIsTearingDown;
	ResetParticles(bEmptyInstances);
	FXSystem = NULL;
	Super::OnUnregister();

	// sanity check
	check(FXSystem == NULL);
}

void UParticleSystemComponent::OnEndOfFrameUpdateDuringTick()
{
	WaitForAsyncAndFinalize(STALL);
}

void UParticleSystemComponent::CreateRenderState_Concurrent(FRegisterComponentContext* Context)
{
	LLM_SCOPE(ELLMTag::Particles);
	SCOPE_CYCLE_COUNTER(STAT_ParticleSystemComponent_CreateRenderState_Concurrent);
	SCOPE_CYCLE_COUNTER(STAT_ParticlesOverview_GT_CNC);

#if WITH_STATE_STREAM_ACTOR
	if (UseParticleSystemStateStream)
	{
		bRenderStateCreated = true;
		return;
	}
#endif

	ForceAsyncWorkCompletion(ENSURE_AND_STALL, false, true);
	check( GetWorld() );
	UE_LOGF(LogParticles,Verbose,
		"CreateRenderState_Concurrent @ %fs %ls", GetWorld()->TimeSeconds,
		Template != NULL ? *Template->GetName() : TEXT("NULL"));

	// NULL out template if we're not allowing particles. This is not done in the Editor to avoid clobbering content via PIE.
	if( !GIsAllowingParticles && !GIsEditor )
	{
		Template = NULL;
	}

	if (Template && Template->bHasPhysics)
	{
		PrimaryComponentTick.TickGroup = TG_PrePhysics;
			
		AEmitter* EmitterOwner = Cast<AEmitter>(GetOwner());
		if (EmitterOwner)
		{
			EmitterOwner->PrimaryActorTick.TickGroup = TG_PrePhysics;
		}
	}

	Super::CreateRenderState_Concurrent(Context);

	bJustRegistered = true;
}



void UParticleSystemComponent::SendRenderTransform_Concurrent()
{
	SCOPE_CYCLE_COUNTER(STAT_ParticleSystemComponent_SendRenderTransform_Concurrent);
	SCOPE_CYCLE_COUNTER(STAT_ParticlesOverview_GT_CNC);

	ForceAsyncWorkCompletion(ENSURE_AND_STALL, false, true);
	if (IsActive())
	{
		if (bSkipUpdateDynamicDataDuringTick == false)
		{
			Super::SendRenderTransform_Concurrent();
			return;
		}
	}
	// skip the Primitive component update to avoid updating the render thread
	UActorComponent::SendRenderTransform_Concurrent();
}

void UParticleSystemComponent::SendRenderDynamicData_Concurrent()
{
	SCOPE_CYCLE_COUNTER(STAT_ParticleSystemComponent_SendRenderDynamicData_Concurrent);
	SCOPE_CYCLE_COUNTER(STAT_ParticlesOverview_GT_CNC);
	CSV_SCOPED_TIMING_STAT_EXCLUSIVE(Effects);
	CSV_SCOPED_TIMING_STAT(Particles, CoreSystems_CascadeSendRenderDynamicData);
	PARTICLE_PERF_STAT_CYCLES_GT(FParticlePerfStatsContext(GetWorld(), Template, this), EndOfFrame);

	ForceAsyncWorkCompletion(ENSURE_AND_STALL, false, true);
	Super::SendRenderDynamicData_Concurrent();

	check(!bAsyncDataCopyIsValid);
	check(!bParallelRenderThreadUpdate);
	bParallelRenderThreadUpdate = true;


	FParticleSystemSceneProxy* PSysSceneProxy = (FParticleSystemSceneProxy*)SceneProxy;
	if (PSysSceneProxy != NULL)
	{
		// check to see if this PSC is active.  When you attach a PSC it gets added to the DataManager
		// even if it might be bIsActive = false  (e.g. attach and later in the frame activate it)
		// or also for PSCs that are attached to a SkelComp which is being attached and reattached but the PSC itself is not active!
		if (IsActive())
		{
			UpdateDynamicData();
		}
		else
		{
			// so if we just were deactivated we want to update the renderer with NULL so the renderer will clear out the data there and not have outdated info which may/will cause a crash
			if (bWasDeactivated || bWasCompleted)
			{
				PSysSceneProxy->UpdateData(NULL);
			}
		}
	}
	bParallelRenderThreadUpdate = false;
}

void UParticleSystemComponent::DestroyRenderState_Concurrent()
{
	SCOPE_CYCLE_COUNTER(STAT_ParticleSystemComponent_DestroyRenderState_Concurrent);
	SCOPE_CYCLE_COUNTER(STAT_ParticlesOverview_GT_CNC);

	ForceAsyncWorkCompletion(ENSURE_AND_STALL, false, true);

	check( GetWorld() );
	UE_LOGF(LogParticles,Verbose,
		"DestroyRenderState_Concurrent @ %fs %ls", GetWorld()->TimeSeconds,
		Template != NULL ? *Template->GetName() : TEXT("NULL"));

	if (bResetOnDetach)
	{
		// Empty the EmitterInstance array.
		ResetParticles();
	}

	if (bRenderStateCreated)
	{
		Super::DestroyRenderState_Concurrent();
	}
}


FDynamicEmitterDataBase* UParticleSystemComponent::CreateDynamicDataFromReplay( FParticleEmitterInstance* EmitterInstance, 
	const FDynamicEmitterReplayDataBase* EmitterReplayData, bool bSelected, ERHIFeatureLevel::Type InFeatureLevel )
{
	checkSlow(EmitterInstance && EmitterInstance->CurrentLODLevel);
	check( EmitterReplayData != NULL );

	FScopeCycleCounterEmitter AdditionalScope(EmitterInstance);
#if WITH_EDITOR
	uint32 StartTime = FPlatformTime::Cycles();
#endif

	// Allocate the appropriate type of emitter data
	FDynamicEmitterDataBase* EmitterData = NULL;

	switch( EmitterReplayData->eEmitterType )
	{
		case DET_Sprite:
			{
				// Allocate the dynamic data
				FDynamicSpriteEmitterData* NewEmitterData = new FDynamicSpriteEmitterData(EmitterInstance->CurrentLODLevel->RequiredModule);

				// Fill in the source data
				const FDynamicSpriteEmitterReplayData* SpriteEmitterReplayData =
					static_cast< const FDynamicSpriteEmitterReplayData* >( EmitterReplayData );
				NewEmitterData->Source = *SpriteEmitterReplayData;

				// Setup dynamic render data.  Only call this AFTER filling in source data for the emitter.
				NewEmitterData->Init( bSelected );

				EmitterData = NewEmitterData;
			}
			break;

		case DET_Mesh:
			{
				// Allocate the dynamic data
				// PVS-Studio does not understand the checkSlow above, so it is warning us that EmitterInstance->CurrentLODLevel may be null.
				FDynamicMeshEmitterData* NewEmitterData = ::new FDynamicMeshEmitterData(EmitterInstance->CurrentLODLevel->RequiredModule);

				// Fill in the source data
				const FDynamicMeshEmitterReplayData* MeshEmitterReplayData =
					static_cast< const FDynamicMeshEmitterReplayData* >( EmitterReplayData );
				NewEmitterData->Source = *MeshEmitterReplayData;

				// Setup dynamic render data.  Only call this AFTER filling in source data for the emitter.

				// @todo: Currently we're assuming the original emitter instance is bound to the same mesh as
				//        when the replay was generated (safe), and various mesh/material indices are intact.  If
				//        we ever support swapping meshes/material on the fly, we'll need cache the mesh
				//        reference and mesh component/material indices in the actual replay data.

				if( EmitterInstance != NULL )
				{
					FParticleMeshEmitterInstance* MeshEmitterInstance =
						static_cast< FParticleMeshEmitterInstance* >( EmitterInstance );
					NewEmitterData->Init(
						bSelected,
						MeshEmitterInstance,
						MeshEmitterInstance->MeshTypeData->Mesh,
						MeshEmitterInstance->MeshTypeData->bUseStaticMeshLODs,
						MeshEmitterInstance->MeshTypeData->LODSizeScale,
						InFeatureLevel);
					EmitterData = NewEmitterData;
				}
			}
			break;

		case DET_Beam2:
			{
				// Allocate the dynamic data
				FDynamicBeam2EmitterData* NewEmitterData = new FDynamicBeam2EmitterData(EmitterInstance->CurrentLODLevel->RequiredModule);

				// Fill in the source data
				const FDynamicBeam2EmitterReplayData* Beam2EmitterReplayData =
					static_cast< const FDynamicBeam2EmitterReplayData* >( EmitterReplayData );
				NewEmitterData->Source = *Beam2EmitterReplayData;

				// Setup dynamic render data.  Only call this AFTER filling in source data for the emitter.
				NewEmitterData->Init( bSelected );

				EmitterData = NewEmitterData;
			}
			break;

		case DET_Ribbon:
			{
				// Allocate the dynamic data
				FDynamicRibbonEmitterData* NewEmitterData = new FDynamicRibbonEmitterData(EmitterInstance->CurrentLODLevel->RequiredModule);

				// Fill in the source data
				const FDynamicRibbonEmitterReplayData* Trail2EmitterReplayData = static_cast<const FDynamicRibbonEmitterReplayData*>(EmitterReplayData);
				NewEmitterData->Source = *Trail2EmitterReplayData;
				// Setup dynamic render data.  Only call this AFTER filling in source data for the emitter.
				NewEmitterData->Init(bSelected);
				EmitterData = NewEmitterData;
			}
			break;

		case DET_AnimTrail:
			{
				// Allocate the dynamic data
				FDynamicAnimTrailEmitterData* NewEmitterData = new FDynamicAnimTrailEmitterData(EmitterInstance->CurrentLODLevel->RequiredModule);
				// Fill in the source data
				const FDynamicTrailsEmitterReplayData* AnimTrailEmitterReplayData = static_cast<const FDynamicTrailsEmitterReplayData*>(EmitterReplayData);
				NewEmitterData->Source = *AnimTrailEmitterReplayData;
				// Setup dynamic render data.  Only call this AFTER filling in source data for the emitter.
				NewEmitterData->Init(bSelected);
				EmitterData = NewEmitterData;
			}
			break;

		default:
			{
				// @todo: Support capture of other particle system types
			}
			break;
	}
#if STATS
	if (EmitterData)
	{
		EmitterData->StatID = EmitterInstance->SpriteTemplate->GetStatIDRT();
	}
#endif

#if WITH_EDITOR
	uint32 EndTime = FPlatformTime::Cycles();
	EmitterInstance->LastTickDurationMs += FPlatformTime::ToMilliseconds(EndTime - StartTime);
#endif

	return EmitterData;
}




FParticleDynamicData* UParticleSystemComponent::CreateDynamicData(ERHIFeatureLevel::Type InFeatureLevel)
{
	//SCOPE_CYCLE_COUNTER(STAT_ParticleSystemComponent_CreateDynamicData);

	FInGameScopedCycleCounter InGameCycleCounter(GetWorld(), EInGamePerfTrackers::VFXSignificance, EInGamePerfTrackerThreads::GameThread, bIsManagingSignificance);

	// Only proceed if we have any live particles or if we're actively replaying/capturing
	if (EmitterInstances.Num() > 0)
	{
		int32 LiveCount = 0;
		for (int32 EmitterIndex = 0; EmitterIndex < EmitterInstances.Num(); EmitterIndex++)
		{
			FParticleEmitterInstance* EmitInst = EmitterInstances[EmitterIndex];
			if (EmitInst)
			{
				if (EmitInst->ActiveParticles > 0)
				{
					LiveCount++;
				}
			}
		}

		if (!bForceLODUpdateFromRenderer
			&& LiveCount == 0
			&& ReplayState == PRS_Disabled)
		{
			return NULL;
		}
	}


	FParticleDynamicData* ParticleDynamicData = new FParticleDynamicData();
	INC_DWORD_STAT(STAT_DynamicPSysCompCount);
	INC_DWORD_STAT_BY(STAT_DynamicPSysCompMem, sizeof(FParticleDynamicData));

	if (Template)
	{
		ParticleDynamicData->SystemPositionForMacroUVs = GetComponentTransform().TransformPosition(Template->MacroUVPosition);
		ParticleDynamicData->SystemRadiusForMacroUVs = Template->MacroUVRadius;
	}

#if WITH_PARTICLE_PERF_STATS
	ParticleDynamicData->PerfStatContext = GetPerfStatsContext();
#endif

	if( ReplayState == PRS_Replaying )
	{
		SCOPE_CYCLE_COUNTER(STAT_ParticleSystemComponent_CreateDynamicData_Replay);
		// Do we have any replay data to play back?
		UParticleSystemReplay* ReplayData = FindReplayClipForIDNumber( ReplayClipIDNumber );
		if( ReplayData != NULL )
		{
			// Make sure the current frame index is in a valid range
			if( ReplayData->Frames.IsValidIndex( ReplayFrameIndex ) )
			{
				// Grab the current particle system replay frame
				const FParticleSystemReplayFrame& CurReplayFrame = ReplayData->Frames[ ReplayFrameIndex ];


				// Fill the emitter dynamic buffers with data from our replay
				ParticleDynamicData->DynamicEmitterDataArray.Reset();
				ParticleDynamicData->DynamicEmitterDataArray.Reserve(CurReplayFrame.Emitters.Num());

				for (int32 CurEmitterIndex = 0; CurEmitterIndex < CurReplayFrame.Emitters.Num(); ++CurEmitterIndex)
				{
					const FParticleEmitterReplayFrame& CurEmitter = CurReplayFrame.Emitters[ CurEmitterIndex ];

					const FDynamicEmitterReplayDataBase* CurEmitterReplay = CurEmitter.FrameState;
					check( CurEmitterReplay != NULL );

					FParticleEmitterInstance* EmitterInst = NULL;
					if( EmitterInstances.IsValidIndex( CurEmitter.OriginalEmitterIndex ) )
					{
						// Fill dynamic data from the replay frame data for this emitter so we can render it
						// Grab the original emitter instance for that this replay was generated from
						FDynamicEmitterDataBase* NewDynamicEmitterData =
							CreateDynamicDataFromReplay( EmitterInstances[ CurEmitter.OriginalEmitterIndex ], CurEmitterReplay, IsOwnerSelected(), InFeatureLevel );


						if( NewDynamicEmitterData != NULL )
						{
							ParticleDynamicData->DynamicEmitterDataArray.Add(NewDynamicEmitterData);
							NewDynamicEmitterData->EmitterIndex = CurEmitterIndex;
						}
					}
				}
			}
		}
	}
	else
	{
		FParticleSystemReplayFrame* NewReplayFrame = NULL;
		if( ReplayState == PRS_Capturing )
		{
			SCOPE_CYCLE_COUNTER(STAT_ParticleSystemComponent_CreateDynamicData_Capture);
			ForceAsyncWorkCompletion(ENSURE_AND_STALL);
			check(IsInGameThread());
			// If we don't have any replay data for this component yet, create some now
			UParticleSystemReplay* ReplayData = FindReplayClipForIDNumber( ReplayClipIDNumber );
			if( ReplayData == NULL )
			{
				// Create a new replay clip!
				ReplayData = NewObject<UParticleSystemReplay>(this);

				// Set the clip ID number
				ReplayData->ClipIDNumber = ReplayClipIDNumber;

				// Add this to the component's list of clips
				ReplayClips.Add( ReplayData );

				// We're modifying the component by adding a new replay clip
				MarkPackageDirty();
			}


			// Add a new frame!
			{
				const int32 NewFrameIndex = ReplayData->Frames.Num();
				new( ReplayData->Frames ) FParticleSystemReplayFrame;
				NewReplayFrame = &ReplayData->Frames[ NewFrameIndex ];

				// We're modifying the component by adding a new frame
				MarkPackageDirty();
			}
		}

		// Is the particle system allowed to run?
		if( bForcedInActive == false )
		{
			//SCOPE_CYCLE_COUNTER(STAT_ParticleSystemComponent_CreateDynamicData_Gather);
			ParticleDynamicData->DynamicEmitterDataArray.Reset();
			ParticleDynamicData->DynamicEmitterDataArray.Reserve(EmitterInstances.Num());

			int32 NumMeshEmitterLODIndices = 0;

			//QUICK_SCOPE_CYCLE_COUNTER(STAT_ParticleSystemComponent_GetDynamicData);
			for (int32 EmitterIndex = 0; EmitterIndex < EmitterInstances.Num(); EmitterIndex++)
			{
				if (SceneProxy)
				{
					++NumMeshEmitterLODIndices;
				}

				FDynamicEmitterDataBase* NewDynamicEmitterData = NULL;
				FParticleEmitterInstance* EmitterInst = EmitterInstances[EmitterIndex];
				if (EmitterInst)
				{
					FScopeCycleCounterEmitter AdditionalScope(EmitterInst);
#if WITH_EDITOR
					uint32 StartTime = FPlatformTime::Cycles();
#endif

					// Generate the dynamic data for this emitter
					{
						//SCOPE_CYCLE_COUNTER(STAT_ParticleSystemComponent_GetDynamicData);
						bool bIsOwnerSeleted = false;
#if WITH_EDITOR
						{
							SCOPE_CYCLE_COUNTER(STAT_ParticleSystemComponent_GetDynamicData_Selected);
							bIsOwnerSeleted = IsOwnerSelected();
						}
#endif
						NewDynamicEmitterData = EmitterInst->GetDynamicData(bIsOwnerSeleted, InFeatureLevel);
					}
					if( NewDynamicEmitterData != NULL )
					{
#if STATS
						NewDynamicEmitterData->StatID = EmitterInst->SpriteTemplate->GetStatIDRT();
#endif
						NewDynamicEmitterData->bValid = true;
						ParticleDynamicData->DynamicEmitterDataArray.Add( NewDynamicEmitterData );
						NewDynamicEmitterData->EmitterIndex = EmitterIndex;

						NewDynamicEmitterData->EmitterIndex = EmitterIndex;
						
						// Are we current capturing particle state?
						if( ReplayState == PRS_Capturing )
						{
							SCOPE_CYCLE_COUNTER(STAT_ParticleSystemComponent_CreateDynamicData_GatherCapture);
							// Capture replay data for this particle system
							// NOTE: This call should always succeed if GetDynamicData succeeded earlier
							FDynamicEmitterReplayDataBase* NewEmitterReplayData = EmitterInst->GetReplayData();
							check( NewEmitterReplayData != NULL );


							// @todo: We could drastically reduce the size of replays in memory and
							//		on disk by implementing delta compression here.

							// Allocate a new emitter frame
							check(NewReplayFrame != NULL);
							const int32 NewFrameEmitterIndex = NewReplayFrame->Emitters.Num();
							new( NewReplayFrame->Emitters ) FParticleEmitterReplayFrame;
							FParticleEmitterReplayFrame* NewEmitterReplayFrame = &NewReplayFrame->Emitters[ NewFrameEmitterIndex ];

							// Store the replay state for this emitter frame.  Note that this will be
							// deleted when the parent object is garbage collected.
							NewEmitterReplayFrame->EmitterType = NewEmitterReplayData->eEmitterType;
							NewEmitterReplayFrame->OriginalEmitterIndex = EmitterIndex;
							NewEmitterReplayFrame->FrameState = NewEmitterReplayData;
						}
					}
#if WITH_EDITOR
					uint32 EndTime = FPlatformTime::Cycles();
					EmitterInst->LastTickDurationMs += FPlatformTime::ToMilliseconds(EndTime - StartTime);
#endif
				}
			}

			if (SceneProxy && static_cast<FParticleSystemSceneProxy*>(SceneProxy)->MeshEmitterLODIndices.Num() != NumMeshEmitterLODIndices)
			{
				ENQUEUE_RENDER_COMMAND(UpdateMeshEmitterLODIndicesCmd)(
					[Proxy = SceneProxy, NumMeshEmitterLODIndices](FRHICommandList&)
				{
					if (Proxy)
					{
						FParticleSystemSceneProxy *ParticleProxy = static_cast<FParticleSystemSceneProxy*>(Proxy);
						ParticleProxy->MeshEmitterLODIndices.Reset();
						ParticleProxy->MeshEmitterLODIndices.AddZeroed(NumMeshEmitterLODIndices);
					}
				});
			}
		}
	}

	return ParticleDynamicData;
}

int32 UParticleSystemComponent::GetNumMaterials() const
{
	if (Template)
	{
		return Template->Emitters.Num();
	}
	return 0;
}

#if WITH_EDITOR
bool UParticleSystemComponent::GetMaterialPropertyPath(int32 ElementIndex, UObject*& OutOwner, FString& OutPropertyPath, FProperty*& OutProperty)
{
	if (EmitterMaterials.IsValidIndex(ElementIndex))
	{
		OutOwner = this;
		OutPropertyPath = FString::Printf(TEXT("%s[%d]"), GET_MEMBER_NAME_STRING_CHECKED(UParticleSystemComponent, EmitterMaterials), ElementIndex);

		if (FArrayProperty* ArrayProperty = CastField<FArrayProperty>(UParticleSystemComponent::StaticClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(UParticleSystemComponent, EmitterMaterials))))
		{
			OutProperty = ArrayProperty->Inner;
		}
		return true;
	}
	if (Template && Template->Emitters.IsValidIndex(ElementIndex))
	{
		UParticleEmitter* Emitter = Template->Emitters[ElementIndex];
		if (Emitter && Emitter->LODLevels.Num() > 0)
		{
			UParticleLODLevel* EmitterLODLevel = Emitter->LODLevels[0];
			if (EmitterLODLevel && EmitterLODLevel->RequiredModule)
			{
				OutOwner = EmitterLODLevel->RequiredModule;
				OutPropertyPath = GET_MEMBER_NAME_STRING_CHECKED(UParticleModuleRequired, Material);
				OutProperty = UParticleModuleRequired::StaticClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(UParticleModuleRequired, Material));
				return true;
			}
		}
	}

	return false;
}
#endif // WITH_EDITOR

UMaterialInterface* UParticleSystemComponent::GetMaterial(int32 ElementIndex) const
{
	if (EmitterMaterials.IsValidIndex(ElementIndex) && EmitterMaterials[ElementIndex] != NULL)
	{
		return EmitterMaterials[ElementIndex];
	}
	if (Template && Template->Emitters.IsValidIndex(ElementIndex))
	{
		UParticleEmitter* Emitter = Template->Emitters[ElementIndex];
		if (Emitter && Emitter->LODLevels.Num() > 0)
		{
			UParticleLODLevel* EmitterLODLevel = Emitter->LODLevels[0];
			if (EmitterLODLevel && EmitterLODLevel->RequiredModule)
			{
				return EmitterLODLevel->RequiredModule->Material;
			}
		}
	}
	return NULL;
}

void UParticleSystemComponent::SetMaterial(int32 ElementIndex, UMaterialInterface* Material)
{
	ForceAsyncWorkCompletion(STALL);
	if (Template && Template->Emitters.IsValidIndex(ElementIndex))
	{
		if (!EmitterMaterials.IsValidIndex(ElementIndex))
		{
			EmitterMaterials.AddZeroed(ElementIndex + 1 - EmitterMaterials.Num());
		}
		EmitterMaterials[ElementIndex] = Material;
		bIsViewRelevanceDirty = true;
	}
	for (int32 EmitterIndex = 0; EmitterIndex < EmitterInstances.Num(); ++EmitterIndex)
	{
		if (FParticleEmitterInstance* Inst = EmitterInstances[EmitterIndex])
		{
			Inst->Tick_MaterialOverrides(EmitterIndex);
		}
	}
	MarkRenderDynamicDataDirty();
	MarkRenderStateDirty();
}

void UParticleSystemComponent::ClearDynamicData()
{
	ForceAsyncWorkCompletion(ENSURE_AND_STALL);
	if (SceneProxy)
	{
		FParticleSystemSceneProxy* ParticleSceneProxy = (FParticleSystemSceneProxy*)SceneProxy;
		ParticleSceneProxy->UpdateData(NULL);
	}
}

void UParticleSystemComponent::UpdateDynamicData()
{
	//SCOPE_CYCLE_COUNTER(STAT_ParticleSystemComponent_UpdateDynamicData);
	LLM_SCOPE(ELLMTag::Particles);

	ForceAsyncWorkCompletion(ENSURE_AND_STALL);
	if (SceneProxy)
	{
		// Create the dynamic data for rendering this particle system
		FParticleDynamicData* ParticleDynamicData = CreateDynamicData(SceneProxy->GetScene().GetFeatureLevel());

		FParticleSystemSceneProxy* Proxy = (FParticleSystemSceneProxy*)SceneProxy;
		// Render the particles
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
		//@todo.SAS. Remove thisline  - it is used for debugging purposes...
		Proxy->SetLastDynamicData(Proxy->GetDynamicData());
		//@todo.SAS. END
		Proxy->SetVisualizeLODIndex(GetCurrentLODIndex());
#endif // !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
		Proxy->UpdateData( ParticleDynamicData );
	}
}

void UParticleSystemComponent::UpdateLODInformation()
{
	ForceAsyncWorkCompletion(ENSURE_AND_STALL);
	if (GetWorld()->IsGameWorld() || (GIsEditor && GEngine->bEnableEditorPSysRealtimeLOD))
	{
		if (SceneProxy)
		{
			if (EmitterInstances.Num() > 0)
			{
				uint8 CheckLODMethod = PARTICLESYSTEMLODMETHOD_DirectSet;
				if (bOverrideLODMethod)
				{
					CheckLODMethod = LODMethod;
				}
				else
				{
					if (Template)
					{
						CheckLODMethod = Template->LODMethod;
					}
				}

				if (CheckLODMethod == PARTICLESYSTEMLODMETHOD_Automatic)
				{
					FParticleSystemSceneProxy* ParticleSceneProxy = (FParticleSystemSceneProxy*)SceneProxy;
					float PendingDistance = ParticleSceneProxy->GetPendingLODDistance();
					if (PendingDistance > 0.0f)
					{
						int32 LODIndex = 0;
						for (int32 LODDistIndex = 1; LODDistIndex < Template->LODDistances.Num(); LODDistIndex++)
						{
							if (Template->LODDistances[LODDistIndex] > ParticleSceneProxy->GetPendingLODDistance())
							{
								break;
							}
							LODIndex = LODDistIndex;
						}

						if (LODIndex != LODLevel)
						{
							SetLODLevel(LODIndex);
						}
					}
				}
			}
		}
	}
	else
	{
#if WITH_EDITORONLY_DATA
		if (LODLevel != EditorLODLevel)
		{
			SetLODLevel(EditorLODLevel);
		}
#endif // WITH_EDITORONLY_DATA
	}
}

void UParticleSystemComponent::OrientZAxisTowardCamera()
{
	SCOPE_CYCLE_COUNTER(STAT_UParticleSystemComponent_OrientZAxisTowardCamera);
	ForceAsyncWorkCompletion(ENSURE_AND_STALL);

	//@TODO: CAMERA: How does this work for stereo and/or split-screen?
	APlayerController* PlayerController = nullptr;
	if (GetWorld() && GetWorld()->GetGameInstance())
	{
		PlayerController = GetWorld()->GetGameInstance()->GetFirstLocalPlayerController();
	}

	// Orient the Z axis toward the camera
	if (PlayerController && PlayerController->PlayerCameraManager)
	{
		// Direction of the camera
		FVector DirToCamera = PlayerController->PlayerCameraManager->GetCameraLocation() - GetComponentLocation();
		DirToCamera.Normalize();

		// Convert the camera direction to local space
		DirToCamera = GetComponentTransform().InverseTransformVectorNoScale(DirToCamera);
		
		// Local Z axis
		const FVector LocalZAxis = FVector(0,0,1);

		// Find angle between z-axis and the camera direction
		const FQuat PointTo = FQuat::FindBetweenNormals(LocalZAxis, DirToCamera);
		
		// Adjust our rotation
		const FRotator AdjustmentAngle(PointTo);
		GetRelativeRotation_DirectMutable() += AdjustmentAngle;

		// Mark the component transform as dirty if the rotation has changed.
		bIsTransformDirty |= !AdjustmentAngle.IsZero();
	}
}

#if WITH_EDITOR
void UParticleSystemComponent::PreEditChange(FProperty* PropertyThatWillChange)
{
	ForceAsyncWorkCompletion(ENSURE_AND_STALL);
	bool bShouldResetParticles = true;

	if (PropertyThatWillChange)
	{
		const FName PropertyName = PropertyThatWillChange->GetFName();

		// Don't reset particles for properties that won't affect the particles
		if (PropertyName == TEXT("bCastVolumetricTranslucentShadow")
			|| PropertyName == TEXT("bCastDynamicShadow")
			|| PropertyName == TEXT("bAffectDynamicIndirectLighting")
			|| PropertyName == TEXT("CastShadow"))
		{
			bShouldResetParticles = false;
		}
	}

	if (bShouldResetParticles)
	{
		ResetParticles();
	}
	
	Super::PreEditChange(PropertyThatWillChange);
}

void UParticleSystemComponent::PostEditChangeChainProperty(FPropertyChangedChainEvent& PropertyChangedEvent)
{
	check(IsInGameThread());
	ForceAsyncWorkCompletion(ENSURE_AND_STALL);
	if (PropertyChangedEvent.PropertyChain.Num() > 0)
	{
		FProperty* MemberProperty = PropertyChangedEvent.PropertyChain.GetActiveMemberNode()->GetValue();
		if ( MemberProperty != NULL )
		{
			FName PropertyName = PropertyChangedEvent.Property->GetFName();
			if ((PropertyName == TEXT("Color")) ||
				(PropertyName == TEXT("R")) ||
				(PropertyName == TEXT("G")) ||
				(PropertyName == TEXT("B")))
			{
				//@todo. once the property code can give the correct index, only update
				// the entry that was actually changed!
				// This function does not return an index into the array at the moment...
				// int32 InstParamIdx = PropertyChangedEvent.GetArrayIndex(TEXT("InstanceParameters"));
				for (int32 InstIdx = 0; InstIdx < InstanceParameters.Num(); InstIdx++)
				{
					FParticleSysParam& PSysParam = InstanceParameters[InstIdx];
					if ((PSysParam.ParamType == PSPT_Vector) || (PSysParam.ParamType == PSPT_VectorRand) || (PSysParam.ParamType == PSPT_VectorUnitRand))
					{
						PSysParam.Vector.X = PSysParam.Color.R / 255.0f;
						PSysParam.Vector.Y = PSysParam.Color.G / 255.0f;
						PSysParam.Vector.Z = PSysParam.Color.B / 255.0f;
					}
				}
			}
		}
	}

	bIsViewRelevanceDirty = true;

	if (ShouldBeTickManaged())
	{
		PrimaryComponentTick.bStartWithTickEnabled = false;
	}

	Super::PostEditChangeChainProperty(PropertyChangedEvent);
}
#endif // WITH_EDITOR

FBoxSphereBounds UParticleSystemComponent::CalcBounds(const FTransform& LocalToWorld) const
{
	FBox BoundingBox;
	BoundingBox.Init();

	// When inactive and using auto attachments do not include our bounds as they will be in an invalid location
	// While active it's more complicated as we could become detatched and wish to play the remainder of the effect so we must include them
	const USceneComponent* UseAutoParent = (bAutoManageAttachment && GetAttachParent() == nullptr) ? AutoAttachParent.Get() : nullptr;
	if (UseAutoParent && !IsActive())
	{
		// We use auto attachment but have detached, don't use our own bogus bounds (we're off near 0,0,0), use the usual parent's bounds.
		return UseAutoParent->Bounds;
	}
	else if (FXConsoleVariables::bAllowCulling == false)
	{
		BoundingBox = FBox(FVector(-HALF_WORLD_MAX), FVector(HALF_WORLD_MAX));
	}
	else if(Template && Template->bUseFixedRelativeBoundingBox)
	{
		// Use hardcoded relative bounding box from template.
		BoundingBox	= Template->FixedRelativeBoundingBox.TransformBy(LocalToWorld);
	}
	else
	{
		for (int32 i=0; i<EmitterInstances.Num(); i++)
		{
			FParticleEmitterInstance* EmitterInstance = EmitterInstances[i];
			if( EmitterInstance && EmitterInstance->HasActiveParticles() )
			{
				BoundingBox += EmitterInstance->GetBoundingBox();
			}
		}

		// If the bounding box is not valid at this point there were no active particles, return zero-extent/radius box at local origin.
		if (!BoundingBox.IsValid)
		{
			return FBoxSphereBounds(LocalToWorld.GetTranslation(), FVector::ZeroVector, 0.0f);
		}

		// Expand the actual bounding-box slightly so it will be valid longer in the case of expanding particle systems.
		const FVector ExpandAmount = BoundingBox.GetExtent() * 0.1f;
		BoundingBox = FBox(BoundingBox.Min - ExpandAmount, BoundingBox.Max + ExpandAmount);
	}

	return FBoxSphereBounds(BoundingBox);
}

class FParticleFinalizeTask
{
	UParticleSystemComponent* Target;
public:
	FParticleFinalizeTask(UParticleSystemComponent* InTarget)
		: Target(InTarget)
	{

	}

	FORCEINLINE TStatId GetStatId() const
	{
		RETURN_QUICK_DECLARE_CYCLE_STAT(FParticleFinalizeTask, STATGROUP_TaskGraphTasks);
	}

	ENamedThreads::Type GetDesiredThread()
	{
		return ENamedThreads::GameThread;
	}

	static ESubsequentsMode::Type GetSubsequentsMode() { return ESubsequentsMode::TrackSubsequents; }

	void DoTask(ENamedThreads::Type CurrentThread, const FGraphEventRef& MyCompletionGraphEvent)
	{
		CSV_SCOPED_TIMING_STAT_EXCLUSIVE(Effects);

		Target->FinalizeTickComponent();
	}
};


TAutoConsoleVariable<int32> CVarFXEarlySchedule(TEXT("FX.EarlyScheduleAsync"), 0, TEXT("If 1, particle system components that can run async will be scheduled earlier in the frame"));

static int32 GBatchParticleAsync = 0;
static FAutoConsoleVariableRef CVarBatchParticleAsync(
	TEXT("FX.BatchAsync"),
	GBatchParticleAsync,
	TEXT("If 1, particle async tasks are batched because they often take less time than it takes to wake up a task thread. No effect on editor.")
);

static int32 GBatchParticleAsyncBatchSize = 32;
static FAutoConsoleVariableRef CVarBatchParticleAsyncBatchSize(
	TEXT("FX.BatchAsyncBatchSize"),
	GBatchParticleAsyncBatchSize,
	TEXT("When FX.BatchAsync = 1, controls the number of particle systems grouped together for threading.")
);


FAutoConsoleTaskPriority CPrio_ParticleAsyncTask(
	TEXT("TaskGraph.TaskPriorities.ParticleAsyncTask"),
	TEXT("Task and thread priority for FParticleAsyncTask."),
	ENamedThreads::HighThreadPriority, // if we have high priority task threads, then use them...
	ENamedThreads::NormalTaskPriority, // .. at normal task priority
	ENamedThreads::HighTaskPriority // if we don't have hi pri threads, then use normal priority threads at high task priority instead
	);


class FParticleAsyncTask
{
	UParticleSystemComponent* Target;
	FGraphEventRef FinalizePrereq;
	FThreadSafeCounter* FinalizeDispatchCounter;

public:
	FParticleAsyncTask(UParticleSystemComponent* InTarget, FGraphEventRef& InFinalizePrereq, FThreadSafeCounter* InFinalizeDispatchCounter)
		: Target(InTarget)
		, FinalizePrereq(InFinalizePrereq)
		, FinalizeDispatchCounter(InFinalizeDispatchCounter)
	{

	}

	FORCEINLINE TStatId GetStatId() const
	{
		RETURN_QUICK_DECLARE_CYCLE_STAT(FParticleAsyncTask, STATGROUP_TaskGraphTasks);
	}

	ENamedThreads::Type GetDesiredThread()
	{
		return CPrio_ParticleAsyncTask.Get();
	}

	static ESubsequentsMode::Type GetSubsequentsMode() { return ESubsequentsMode::TrackSubsequents; }

	void DoTask(ENamedThreads::Type CurrentThread, const FGraphEventRef& MyCompletionGraphEvent)
	{
		FTaskTagScope Scope(ETaskTag::EParallelRenderingThread);
		Target->ComputeTickComponent_Concurrent();
#if !WITH_EDITOR  // otherwise this is queued by the calling code because we need to be able to block and wait on it
		{
			SCOPE_CYCLE_COUNTER(STAT_UParticleSystemComponent_QueueFinalize);
			FGraphEventArray Prereqs;
			if (FinalizePrereq.GetReference())
			{
				Prereqs.Add(FinalizePrereq);
			}
			FGraphEventRef Finalize = TGraphTask<FParticleFinalizeTask>::CreateTask(&Prereqs, CurrentThread).ConstructAndDispatchWhenReady(Target);
			MyCompletionGraphEvent->DontCompleteUntil(Finalize);
			if (FinalizeDispatchCounter)
			{
				if (FinalizeDispatchCounter->Decrement() == 0)
				{
					check(FinalizePrereq.GetReference() && !FinalizePrereq->IsComplete());
					{
						FinalizePrereq->DispatchSubsequents();
					}
					delete FinalizeDispatchCounter;
				}
			}
		}
#endif
	}
};



class FDispatchBatchedAsyncTasks
{
	FGraphEventRef Target;
public:

	FDispatchBatchedAsyncTasks(FGraphEventRef& InTarget)
		: Target(InTarget)
	{
	}

	FORCEINLINE TStatId GetStatId() const
	{
		RETURN_QUICK_DECLARE_CYCLE_STAT(FDispatchBatchedAsyncTasks, STATGROUP_TaskGraphTasks);
	}

	ENamedThreads::Type GetDesiredThread()
	{
		return CPrio_ParticleAsyncTask.Get();
	}

	static ESubsequentsMode::Type GetSubsequentsMode() { return ESubsequentsMode::FireAndForget; }

	void DoTask(ENamedThreads::Type CurrentThread, const FGraphEventRef& MyCompletionGraphEvent)
	{
		check(Target.GetReference() && !Target->IsComplete());
		{
			Target->DispatchSubsequents();
		}
	}
};

class FGameThreadDispatchBatchedAsyncTasks
{
	FGraphEventRef Target;
public:
	FGameThreadDispatchBatchedAsyncTasks(FGraphEventRef& InTarget)
		: Target(InTarget)
	{

	}

	FORCEINLINE TStatId GetStatId() const
	{
		RETURN_QUICK_DECLARE_CYCLE_STAT(FGameThreadDispatchBatchedAsyncTasks, STATGROUP_TaskGraphTasks);
	}

	ENamedThreads::Type GetDesiredThread()
	{
		return ENamedThreads::GameThread;
	}

	static ESubsequentsMode::Type GetSubsequentsMode() { return ESubsequentsMode::FireAndForget; }

	void DoTask(ENamedThreads::Type CurrentThread, const FGraphEventRef& MyCompletionGraphEvent);
};


struct FFXAsyncBatcher
{
	FGraphEventArray DispatchEvent;
	FGraphEventRef FinalizeOnGTDispatchEvent;
	FThreadSafeCounter* FinalizeDispatchCounter;
	int32 NumBatched = 0;


	FGraphEventArray* GetAsyncPrereq(FGraphEventRef& OutFinalizeBatchEvent, FThreadSafeCounter*& OutFinalizeDispatchCounter)
	{
		check(IsInGameThread());
#if !WITH_EDITOR
		if (GBatchParticleAsync)
		{
			if (NumBatched >= GBatchParticleAsyncBatchSize || !DispatchEvent.Num() || !DispatchEvent[0].GetReference() || DispatchEvent[0]->IsComplete())
			{
				Flush();
			}
			if (DispatchEvent.Num() == 0)
			{
				check(NumBatched == 0 && !FinalizeDispatchCounter && !FinalizeOnGTDispatchEvent.GetReference());
				DispatchEvent.Add(FGraphEvent::CreateGraphEvent());
				FinalizeOnGTDispatchEvent = FGraphEvent::CreateGraphEvent();
				TGraphTask<FGameThreadDispatchBatchedAsyncTasks>::CreateTask(nullptr, ENamedThreads::GameThread).ConstructAndDispatchWhenReady(DispatchEvent[0]);
				check(!FinalizeDispatchCounter);
				FinalizeDispatchCounter = new FThreadSafeCounter();
			}
			OutFinalizeBatchEvent = FinalizeOnGTDispatchEvent;
			FinalizeDispatchCounter->Increment();
			OutFinalizeDispatchCounter = FinalizeDispatchCounter;
			NumBatched++;
			return &DispatchEvent;
		}
#endif
		check(!OutFinalizeBatchEvent.GetReference() && !OutFinalizeDispatchCounter);
		return nullptr;
	}

	void Flush()
	{
		if (NumBatched)
		{
			check(FinalizeDispatchCounter && FinalizeDispatchCounter->GetValue() == NumBatched);
			check(DispatchEvent.Num() && DispatchEvent[0].GetReference() && !DispatchEvent[0]->IsComplete());
			{
				TGraphTask<FDispatchBatchedAsyncTasks>::CreateTask(nullptr, ENamedThreads::GameThread).ConstructAndDispatchWhenReady(DispatchEvent[0]);
			}

			FinalizeOnGTDispatchEvent = nullptr;
			DispatchEvent.Empty();
			NumBatched = 0;
			FinalizeDispatchCounter = nullptr; // deleted by the last task
		}
	}

};

static FFXAsyncBatcher FXAsyncBatcher;

void FGameThreadDispatchBatchedAsyncTasks::DoTask(ENamedThreads::Type CurrentThread, const FGraphEventRef& MyCompletionGraphEvent)
{
	check(IsInGameThread());
	FXAsyncBatcher.Flush();
}


bool UParticleSystemComponent::IsReadyForOwnerToAutoDestroy() const
{
	return (!IsActive() && bWasCompleted);
}

void UParticleSystemComponent::SetComponentTickEnabled(bool bEnabled)
{
	//Never enable the tick if we're not registered.
	bEnabled &= IsRegistered();
	check(!bEnabled || GetWorld());

	bool bShouldTickBeManaged = ShouldBeTickManaged();
	bool bIsTickManaged = IsTickManaged();
	FParticleSystemWorldManager* PSCMan = bShouldTickBeManaged || bIsTickManaged ? GetWorldManager() : nullptr;

	if ((bShouldTickBeManaged || bIsTickManaged) && PSCMan == nullptr)
	{
		Super::SetComponentTickEnabled(bEnabled);
		return;
	}

	if (bShouldTickBeManaged)
	{
		Super::SetComponentTickEnabled(false);//Ensure we're not ticking via task graph.
		if (bEnabled)
		{
			if (!PSCMan->RegisterComponent(this))
			{
				UE_LOGF(LogParticles, Error, "Failed to register with the PSC world manager");
			}
		}
		else if (bIsTickManaged)
		{
			PSCMan->UnregisterComponent(this);
		}
	}
	else
	{
		//Make sure we're not ticking via the manager.
		if (bIsTickManaged)
		{
			PSCMan->UnregisterComponent(this);
		}

		Super::SetComponentTickEnabled(bEnabled);
	}
}

bool UParticleSystemComponent::IsComponentTickEnabled()const
{
	//As far as anyone else is concerned, a tick managed component is ticking. The shouldn't know or care how.
	return Super::IsComponentTickEnabled() || IsTickManaged();
}

void UParticleSystemComponent::OnAttachmentChanged()
{
	Super::OnAttachmentChanged();

	if (IsTickManaged())
	{
		// Note: the PSCMan can become invalid during GC / level change
		if (FParticleSystemWorldManager* PSCMan = GetWorldManager())
		{
			//Reregister component to recalculate dependencies and re add to manager's lists.
			PSCMan->UnregisterComponent(this);
			PSCMan->RegisterComponent(this);
		}
	}
}

void UParticleSystemComponent::OnChildAttached(USceneComponent* ChildComponent)
{
	Super::OnChildAttached(ChildComponent);
	if (IsComponentTickEnabled())
	{
		//This will ensure we're set to be ticking via the correct path.
		//If we can, we should move to being tick managed. If not, we should move to regular tick.
		//Having attached children is currently a disqualifying state for PSCs, if this changes we may need to also have some reregistration mechanics here so dependancies can be recalculated.
		SetComponentTickEnabled(true);
	}
}

void UParticleSystemComponent::OnChildDetached(USceneComponent* ChildComponent)
{
	Super::OnChildDetached(ChildComponent);
	if (IsComponentTickEnabled())
	{
		//This will ensure we're set to be ticking via the correct path.
		//If we can, we should move to being tick managed. If not, we should move to regular tick.
		//Having attached children is currently a disqualifying state for PSCs, if this changes we may need to also have some reregistration mechanics here so dependancies can be recalculated.
		SetComponentTickEnabled(true);
	}
}

void UParticleSystemComponent::TickComponent(float DeltaTime, enum ELevelTick TickType, FActorComponentTickFunction *ThisTickFunction)
{
	CSV_SCOPED_TIMING_STAT_EXCLUSIVE(Effects);
	LLM_SCOPE(ELLMTag::Particles);
	FInGameScopedCycleCounter InGameCycleCounter(GetWorld(), EInGamePerfTrackers::VFXSignificance, EInGamePerfTrackerThreads::GameThread, bIsManagingSignificance);
	SCOPE_CYCLE_COUNTER(STAT_ParticlesOverview_GT);
	FScopeCycleCounterUObject AdditionalScope(AdditionalStatObject(), GET_STATID(STAT_ParticlesOverview_GT));

	if (Template == nullptr || Template->Emitters.Num() == 0)
	{
		// Disable our tick here, will be enabled when activating
		SetComponentTickEnabled(false);
		return;
	}

	PARTICLE_PERF_STAT_CYCLES_WITH_COUNT_GT(FParticlePerfStatsContext(GetWorld(), Template, this), TickGameThread, 1);

	checkf(!IsTickManaged() || !PrimaryComponentTick.IsTickFunctionEnabled(), TEXT("PSC has enabled tick funciton and is also ticking via the tick manager.\nTemplate:%s\nPSC: %s\nParent:%s")
	, *Template->GetFullName(), *GetFullName(), GetAttachParent() ? *GetAttachParent()->GetFullName() : TEXT("nullptr"));

	// control tick rate
	// don't tick if enough time hasn't passed
	if (TimeSinceLastTick + static_cast<uint32>(DeltaTime*1000.0f) < Template->MinTimeBetweenTicks)
	{
		TimeSinceLastTick += static_cast<uint32>(DeltaTime*1000.0f);
		return;
	}
	// if enough time has passed, and some of it in previous frames, need to take that into account for DeltaTime
	DeltaTime += TimeSinceLastTick / 1000.0f;
	TimeSinceLastTick = 0;

	if (bDeactivateTriggered)
	{
		DeactivateSystem();

		if (bWasDeactivated)
		{
			OnComponentDeactivated.Broadcast(this);
		}
	}

	ForceAsyncWorkCompletion(ENSURE_AND_STALL);
	SCOPE_CYCLE_COUNTER(STAT_PSysCompTickTime);

	if (bWasManagingSignificance != bIsManagingSignificance)
	{	
		bWasManagingSignificance = bIsManagingSignificance;
		MarkRenderStateDirty();
	}

	bool bDisallowAsync = false;

	// Bail out if inactive and not AutoActivate
	if ((IsActive() == false) && (bAutoActivate == false))
	{
		// Disable our tick here, will be enabled when activating
		SetComponentTickEnabled(false);
		return;
	}
	DeltaTimeTick = DeltaTime;

	// Bail out if we are running on a dedicated server and we don't want to update on those
	if ((bUpdateOnDedicatedServer == false) && (IsNetMode(NM_DedicatedServer)))
	{
		if (bAutoDestroy)
		{
			// We need to destroy the component if the user is expecting us to do it automatically otherwise this component will live forever because HasCompleted() will never get checked
			DestroyComponent();
		}
		else
		{
			SetComponentTickEnabled(false);
		}
		return;
	}

	UWorld* World = GetWorld();
	check(World);

	bool bRequiresReset = bResetTriggered;
	bResetTriggered = false;

	// System settings may have been lowered. Support late deactivation.
	int32 DetailModeCVar = GetCurrentDetailMode();
	const bool bDetailModeAllowsRendering	= DetailMode <= DetailModeCVar;
	if (bDetailModeAllowsRendering == false)
	{
		if (IsActive())
		{
			DeactivateSystem();
			Super::MarkRenderDynamicDataDirty();
		}
		return;
	} 
	
	// Has the actor position changed to the point where we need to reset the LWC tile
	if (RequiresLWCTileRecache(LWCTile, GetComponentLocation()))
	{
		//-OPT: We may be able to narrow down when a reset is required, like having a GPU emitter, having world space emitters, etc.
		//      Cascade generally operates at double precision so it may only be GPU emitters that require a reset.
		UE_LOGF(LogParticles, Warning, "PSC(%ls - %ls) required LWC tile recache and was reset.", *GetFullNameSafe(this), *GetFullNameSafe(Template));
		bRequiresReset = true;
	}

	if (bRequiresReset)
	{
		ForceReset();
	}

	// Bail out if MaxSecondsBeforeInactive > 0 and we haven't been rendered the last MaxSecondsBeforeInactive seconds.
	if (bWarmingUp == false)
	{
		//For now, we're only allowing the SecondsBeforeInactive optimization on looping emitters as it can cause leaks with non-looping effects.
		//Longer term, there is likely a better solution.
		if (CanSkipTickDueToVisibility())//Cannot skip ticking if we've been deactivated otherwise the system cannot complete correctly.
		{
			return;
		}

		AccumLODDistanceCheckTime += DeltaTime;
		if (AccumLODDistanceCheckTime > Template->LODDistanceCheckTime)
		{
			SCOPE_CYCLE_COUNTER(STAT_UParticleSystemComponent_LOD);
			AccumLODDistanceCheckTime = 0.0f;

			if (ShouldComputeLODFromGameThread())
			{
				bool bCalculateLODLevel = 
					(bOverrideLODMethod == true) ? (LODMethod == PARTICLESYSTEMLODMETHOD_Automatic) : 
						(Template->LODMethod == PARTICLESYSTEMLODMETHOD_Automatic);
				if (bCalculateLODLevel == true)
				{
					FVector EffectPosition = GetComponentLocation();
					int32 DesiredLODLevel = DetermineLODLevelForLocation(EffectPosition);
					SetLODLevel(DesiredLODLevel);
				}
			}
			else
			{
				// Periodically force an LOD update from the renderer if we are
				// using rendering results to make LOD decisions.
				bForceLODUpdateFromRenderer = true;
				UpdateLODInformation();
			}
		}
	}

	bForcedInActive = false;

	DeltaTime *= CustomTimeDilation;
	DeltaTimeTick = DeltaTime;
	if (FMath::IsNearlyZero(DeltaTimeTick) && GFXSkipZeroDeltaTime)
	{
		return;
	}

	AccumTickTime += DeltaTime;

	// Save player locations
	PlayerLocations.Reset();
	PlayerLODDistanceFactor.Reset();

#if WITH_EDITOR
	// clear tick timers
	for (auto Instance : EmitterInstances)
	{
		if (Instance)
		{
			Instance->LastTickDurationMs = 0.0f;
		}
	}
#endif

	if (World->IsGameWorld())
	{
		for (FConstPlayerControllerIterator Iterator = World->GetPlayerControllerIterator(); Iterator; ++Iterator)
		{
			APlayerController* PlayerController = Iterator->Get();
			if (PlayerController && PlayerController->IsLocalPlayerController())
			{
				FVector POVLoc;
				FRotator POVRotation;
				PlayerController->GetPlayerViewPoint(POVLoc, POVRotation);

				PlayerLocations.Add(POVLoc);
				PlayerLODDistanceFactor.Add(PlayerController->LocalPlayerCachedLODDistanceFactor);
			}
		}
	}

	// Orient the Z axis toward the camera
	if (Template->bOrientZAxisTowardCamera)
	{
		OrientZAxisTowardCamera();
	}

	if (Template->SystemUpdateMode == EPSUM_FixedTime)
	{
		// Use the fixed delta time!
		DeltaTimeTick = Template->UpdateTime_Delta;
	}

	// Clear out the events.
	SpawnEvents.Reset();
	DeathEvents.Reset();
	CollisionEvents.Reset();
	BurstEvents.Reset();
	TotalActiveParticles = 0;
	bNeedsFinalize = true;
	
	if (!IsTickManaged() || bWarmingUp)
	{
		if (!ThisTickFunction || !ThisTickFunction->IsCompletionHandleValid() || !CanTickInAnyThread() || FXConsoleVariables::bFreezeParticleSimulation || !FXConsoleVariables::bAllowAsyncTick || !FApp::ShouldUseThreadingForPerformance() ||
			GDistributionType == 0) // this may not be absolutely required, however if you are using distributions it will be glacial anyway. If you want to get rid of this, note that some modules use this indirectly as their criteria for CanTickInAnyThread
		{
			bDisallowAsync = true;
		}

		if (bDisallowAsync)
		{
			if (!FXConsoleVariables::bFreezeParticleSimulation)
			{
				ComputeTickComponent_Concurrent();
			}
			FinalizeTickComponent();
		}
		else
		{
			SCOPE_CYCLE_COUNTER(STAT_UParticleSystemComponent_QueueTasks);

			MarshalParamsForAsyncTick();
			{
				SCOPE_CYCLE_COUNTER(STAT_UParticleSystemComponent_QueueAsync);
				FGraphEventRef OutFinalizeBatchEvent;
				FThreadSafeCounter* FinalizeDispatchCounter = nullptr;
				FGraphEventArray* Prereqs = FXAsyncBatcher.GetAsyncPrereq(OutFinalizeBatchEvent, FinalizeDispatchCounter);
				AsyncWork = TGraphTask<FParticleAsyncTask>::CreateTask(Prereqs, ENamedThreads::GameThread).ConstructAndDispatchWhenReady(this, OutFinalizeBatchEvent, FinalizeDispatchCounter);
#if !WITH_EDITOR  // we need to not complete until this is done because the game thread finalize task has not beed queued yet
				ThisTickFunction->GetCompletionHandle()->DontCompleteUntil(AsyncWork);
#endif
			}
#if WITH_EDITOR  // we need to queue this here because we need to be able to block and wait on it
			{
				SCOPE_CYCLE_COUNTER(STAT_UParticleSystemComponent_QueueFinalize);
				FGraphEventArray Prereqs;
				Prereqs.Add(AsyncWork);
				FGraphEventRef Finalize = TGraphTask<FParticleFinalizeTask>::CreateTask(&Prereqs, ENamedThreads::GameThread).ConstructAndDispatchWhenReady(this);
				ThisTickFunction->GetCompletionHandle()->DontCompleteUntil(Finalize);
			}
#endif

			if (CVarFXEarlySchedule.GetValueOnGameThread())
			{
				PrimaryComponentTick.TickGroup = TG_PrePhysics;
				PrimaryComponentTick.EndTickGroup = TG_PostPhysics;
			}
			else
			{
				PrimaryComponentTick.TickGroup = TG_DuringPhysics;
			}
		}
	}
}

int32 UParticleSystemComponent::GetCurrentDetailMode() const
{
#if WITH_EDITORONLY_DATA
	if (!GEngine->bEnableEditorPSysRealtimeLOD && EditorDetailMode >= 0)
	{
		return EditorDetailMode;
	}
	else
#endif
	{
		return GetCachedScalabilityCVars().DetailMode;
	}
}

void UParticleSystemComponent::ComputeTickComponent_Concurrent()
{
	FInGameScopedCycleCounter InGameCycleCounter(GetWorld(), EInGamePerfTrackers::VFXSignificance, IsInGameThread() ? EInGamePerfTrackerThreads::GameThread : EInGamePerfTrackerThreads::OtherThread, bIsManagingSignificance);

	SCOPE_CYCLE_COUNTER(STAT_ParticleComputeTickTime);
	FScopeCycleCounterUObject AdditionalScope(AdditionalStatObject(), GET_STATID(STAT_ParticleComputeTickTime));
	SCOPE_CYCLE_COUNTER(STAT_ParticlesOverview_GT_CNC);
	PARTICLE_PERF_STAT_CYCLES_GT(FParticlePerfStatsContext(GetWorld(), Template, this), TickConcurrent);

	// Tick Subemitters.
	int32 EmitterIndex;
	NumSignificantEmitters = 0;
	for (EmitterIndex = 0; EmitterIndex < EmitterInstances.Num(); EmitterIndex++)
	{
		FParticleEmitterInstance* Instance = EmitterInstances[EmitterIndex];
		FScopeCycleCounterEmitter AdditionalScopeInner(Instance);
#if WITH_EDITOR
		uint32 StartTime = FPlatformTime::Cycles();
#endif

		if (EmitterIndex + 1 < EmitterInstances.Num())
		{
			FParticleEmitterInstance* NextInstance = EmitterInstances[EmitterIndex+1];
			FPlatformMisc::Prefetch(NextInstance);
		}

		if (Instance && Instance->SpriteTemplate)
		{
			check(Instance->SpriteTemplate->LODLevels.Num() > 0);

			UParticleLODLevel* SpriteLODLevel = Instance->SpriteTemplate->GetCurrentLODLevel(Instance);
			if (SpriteLODLevel && SpriteLODLevel->bEnabled)
			{
				if (bIsManagingSignificance)
				{
					bool bEmitterIsSignificant = Instance->SpriteTemplate->IsSignificant(RequiredSignificance);
					if (bEmitterIsSignificant)
					{
						++NumSignificantEmitters;
						Instance->SetHaltSpawning(false);
						Instance->SetFakeBurstWhenSpawningSupressed(false);
						Instance->bEnabled = true;
					}
					else
					{
						Instance->SetHaltSpawning(true);
						Instance->SetFakeBurstWhenSpawningSupressed(true);
						if (Instance->SpriteTemplate->bDisableWhenInsignficant)
						{
							Instance->bEnabled = false;
						}
					}
				}
				else
				{
					++NumSignificantEmitters;
				}

				Instance->Tick(DeltaTimeTick, bSuppressSpawning);

				Instance->Tick_MaterialOverrides(EmitterIndex);
				TotalActiveParticles += Instance->ActiveParticles;
			}

#if WITH_EDITOR
			uint32 EndTime = FPlatformTime::Cycles();
			Instance->LastTickDurationMs += FPlatformTime::ToMilliseconds(EndTime - StartTime);
#endif
		}
	}
	if (bAsyncWorkOutstanding)
	{
		FPlatformMisc::MemoryBarrier();
		bAsyncWorkOutstanding = false;
	}
}

void UParticleSystemComponent::FinalizeTickComponent()
{
	FInGameScopedCycleCounter InGameCycleCounter(GetWorld(), EInGamePerfTrackers::VFXSignificance, IsInGameThread() ? EInGamePerfTrackerThreads::GameThread : EInGamePerfTrackerThreads::OtherThread, bIsManagingSignificance);

	SCOPE_CYCLE_COUNTER(STAT_ParticleFinalizeTickTime);
	SCOPE_CYCLE_COUNTER(STAT_ParticlesOverview_GT);
	PARTICLE_PERF_STAT_CYCLES_GT(FParticlePerfStatsContext(GetWorld(), Template, this), Finalize);

	if(bAsyncDataCopyIsValid)
	{
		//reset async actor to world
		for (FParticleSysParam& ParticleSysParam : AsyncInstanceParameters)
		{
			ParticleSysParam.ResetAsyncActorCache();
		}
	}

	bAsyncDataCopyIsValid = false;
	AsyncWork = nullptr; // this task is done
	if (!bNeedsFinalize)
	{
		return;
	}
	bNeedsFinalize = false;

	if (FXConsoleVariables::bFreezeParticleSimulation == false)
	{
		int32 EmitterIndex;
		// Now, process any events that have occurred.
		for (EmitterIndex = 0; EmitterIndex < EmitterInstances.Num(); EmitterIndex++)
		{
			FParticleEmitterInstance* Instance = EmitterInstances[EmitterIndex];
			if (Instance && Instance->bEnabled)
			{
				if (EmitterIndex + 1 < EmitterInstances.Num())
				{
					FParticleEmitterInstance* NextInstance = EmitterInstances[EmitterIndex+1];
					FPlatformMisc::Prefetch(NextInstance);
				}

				if (Instance->SpriteTemplate)
				{
					UParticleLODLevel* SpriteLODLevel = Instance->SpriteTemplate->GetCurrentLODLevel(Instance);
					if (SpriteLODLevel && SpriteLODLevel->bEnabled)
					{
						Instance->ProcessParticleEvents(DeltaTimeTick, bSuppressSpawning);
					}
				}
			}
		}

		UWorld* World = GetWorld();
		AParticleEventManager* EventManager = (World ? ToRawPtr(World->MyParticleEventManager) : NULL);
		if (EventManager)
		{
			if (SpawnEvents.Num() > 0) EventManager->HandleParticleSpawnEvents(this, SpawnEvents);
			if (DeathEvents.Num() > 0) EventManager->HandleParticleDeathEvents(this, DeathEvents);
			if (CollisionEvents.Num() > 0) EventManager->HandleParticleCollisionEvents(this, CollisionEvents);
			if (BurstEvents.Num() > 0) EventManager->HandleParticleBurstEvents(this, BurstEvents);
		}
	}
	// Clear out the Kismet events, as they should have been processed by now...
	KismetEvents.Empty();

	// Indicate that we have been ticked since being registered.
	bJustRegistered = false;

	float CurrTime = GetWorld()->GetTimeSeconds();

	//Are we still significant?
	if ((IsActive() && !bWasDeactivated) && bIsManagingSignificance && NumSignificantEmitters == 0 && CurrTime >= LastSignificantTime + Template->InsignificanceDelay)
	{
		OnSignificanceChanged(false, true);
	}
	else
	{
		LastSignificantTime = CurrTime;
		// If component has just totally finished, call script event.
		const bool bIsCompleted = HasCompleted(); 
		if (bIsCompleted && !bWasCompleted)
		{
			Complete();
		}
		bWasCompleted = bIsCompleted;
	}

	// Update bounding box.
	if (!bWarmingUp && !bWasCompleted && !Template->bUseFixedRelativeBoundingBox && !bIsTransformDirty)
	{
		// Force an update every once in a while to shrink the bounds.
		TimeSinceLastForceUpdateTransform += DeltaTimeTick;
		if(TimeSinceLastForceUpdateTransform > MaxTimeBeforeForceUpdateTransform)
		{
			bIsTransformDirty = true;
		}
		else
		{
			// Compute the new system bounding box.
			FBox BoundingBox;
			BoundingBox.Init();

			for (int32 i=0; i<EmitterInstances.Num(); i++)
			{
				FParticleEmitterInstance* Instance = EmitterInstances[i];
				if (Instance && Instance->SpriteTemplate)
				{
					UParticleLODLevel* SpriteLODLevel = Instance->SpriteTemplate->GetCurrentLODLevel(Instance);
					if (SpriteLODLevel && SpriteLODLevel->bEnabled)
					{
						BoundingBox += Instance->GetBoundingBox();
					}
				}
			}

			// Only update the primitive's bounding box in the octree if the system bounding box has gotten larger.
			if(!Bounds.GetBox().IsInside(BoundingBox.Min) || !Bounds.GetBox().IsInside(BoundingBox.Max))
			{
				bIsTransformDirty = true;
			}
		}
	}

	// Update if the component transform has been dirtied.
	if(bIsTransformDirty)
	{
		UpdateComponentToWorld();
		
		TimeSinceLastForceUpdateTransform = 0.0f;
		bIsTransformDirty = false;
	}

	if (bOldPositionValid)
	{
		const float InvDeltaTime = (DeltaTimeTick > 0.0f) ? 1.0f / DeltaTimeTick : 0.0f;
		PartSysVelocity = (GetComponentLocation() - OldPosition) * InvDeltaTime;
	}
	else
	{
		PartSysVelocity = FVector::ZeroVector;
	}
	bOldPositionValid = true;
	OldPosition = GetComponentLocation();

	if (bIsViewRelevanceDirty)
	{
		ConditionalCacheViewRelevanceFlags();
	}

	if (bSkipUpdateDynamicDataDuringTick == false)
	{
		Super::MarkRenderDynamicDataDirty();
	}

}

void UParticleSystemComponent::WaitForAsyncAndFinalize(EForceAsyncWorkCompletion Behavior, bool bDefinitelyGameThread) const
{
	if (AsyncWork.GetReference() && !AsyncWork->IsComplete())
	{
		bool bIsInGameThread = bDefinitelyGameThread || IsInGameThread();
		if (bIsInGameThread)
		{
			FXAsyncBatcher.Flush();
		}
		double StartTime = FPlatformTime::Seconds();
		if (bDefinitelyGameThread)
		{
			check(IsInGameThread());
			SCOPE_CYCLE_COUNTER(STAT_GTSTallTime);
			SCOPE_CYCLE_COUNTER(STAT_UParticleSystemComponent_WaitForAsyncAndFinalize);
			PARTICLE_PERF_STAT_CYCLES_GT(FParticlePerfStatsContext(GetWorld(), Template, this), Wait);
			
			if(WITH_EDITOR && !IsTickManaged())
			{
				FTaskGraphInterface::Get().WaitUntilTaskCompletes(AsyncWork, ENamedThreads::GameThread_Local);
			}

			// since in the non-editor case the completion is chained to a game thread task (not a gamethread_local one), and we don't want to execute arbitrary tasks 
			// in what is probably a very, very deep callstack, we will spin here and wait for the async task to finish. The we will do the finalize. The finalize will be attempted again later but do nothing
			while (bAsyncWorkOutstanding)
			{
				FPlatformProcess::SleepNoStats(0.0f);
			}
		}
		else
		{
			SCOPE_CYCLE_COUNTER(STAT_UParticleSystemComponent_WaitForAsyncAndFinalize);
			PARTICLE_PERF_STAT_CYCLES_GT(FParticlePerfStatsContext(GetWorld(), Template, this), Wait);
			while (bAsyncWorkOutstanding)
			{
				FPlatformProcess::SleepNoStats(0.0f);
			}
		}

		//if (bDelayTick && IsTickManaged())
		//{
			//TODO: If we're completing early for a activate/deactivate etc call from some external owner and it stalls us, we can possible reduce stall chance by telling the PSC manager to move us into a later tick group?
		//}

		float ThisTime = float(FPlatformTime::Seconds() - StartTime) * 1000.0f;
		if (Behavior != SILENT && ThisTime >= 3.0f)
		{
			if (bIsInGameThread)
			{
				UE_LOGF(LogParticles, Warning, "Stalled gamethread waiting for particles %5.6fms '%ls' '%ls'", ThisTime, *GetFullNameSafe(this), *GetFullNameSafe(Template));
			}
			else
			{
				UE_LOGF(LogParticles, Warning, "Stalled worker thread waiting for particles %5.6fms '%ls' '%ls'", ThisTime, *GetFullNameSafe(this), *GetFullNameSafe(Template));
			}
		}
		const_cast<UParticleSystemComponent*>(this)->FinalizeTickComponent();
	}
}

void UParticleSystemComponent::InitParticles()
{
	LLM_SCOPE(ELLMTag::Particles);

	SCOPE_CYCLE_COUNTER(STAT_ParticleSystemComponent_InitParticles);

	if (IsTemplate() == true)
	{
		return;
	}
	ForceAsyncWorkCompletion(ENSURE_AND_STALL);

	check(GetWorld());
	UE_LOGF(LogParticles,Verbose,
		"InitParticles @ %fs %ls", GetWorld()->TimeSeconds,
		Template != NULL ? *Template->GetName() : TEXT("NULL"));

	if (Template != NULL)
	{
		WarmupTime = Template->WarmupTime;
		WarmupTickRate = Template->WarmupTickRate;
		bIsViewRelevanceDirty = true;
		const int32 GlobalDetailMode = GetCurrentDetailMode();
		const bool bCanEverRender = CanEverRender();

		//simplified version.
		int32 NumInstances = EmitterInstances.Num();
		int32 NumEmitters = Template->Emitters.Num();
		const bool bIsFirstCreate = NumInstances == 0;
		EmitterInstances.SetNumZeroed(NumEmitters);

		bWasCompleted = bIsFirstCreate ? false : bWasCompleted;

		bool bClearDynamicData = false;
		int32 PreferredLODLevel = LODLevel;
		bool bSetLodLevels = LODLevel > 0; //We should set the lod level even when creating all emitters if the requested LOD is not 0. 

		for (int32 Idx = 0; Idx < NumEmitters; Idx++)
		{
			UParticleEmitter* Emitter = Template->Emitters[Idx];
			if (Emitter)
			{
				FParticleEmitterInstance* Instance = NumInstances == 0 ? NULL : EmitterInstances[Idx];
				check(GlobalDetailMode < EParticleDetailMode::PDM_MAX);
				const bool bDetailModeAllowsRendering = DetailMode <= GlobalDetailMode && (Emitter->DetailModeBitmask & (1 << GlobalDetailMode));
				const bool bShouldCreateAndOrInit = bDetailModeAllowsRendering && Emitter->HasAnyEnabledLODs() && bCanEverRender;

				if (bShouldCreateAndOrInit)
				{
					if (Instance)
					{
						Instance->SetHaltSpawning(false);
						Instance->SetHaltSpawningExternal(false);
					}
					else
					{
						if (!InstanceOwner)
						{
							InstanceOwner = MakeUnique<FInstanceOwner>(this);
						}

						Instance = Emitter->CreateInstance(*InstanceOwner);
						EmitterInstances[Idx] = Instance;
					}

					if (Instance)
					{
						Instance->bEnabled = true;
						Instance->InitParameters(Emitter);
						Instance->Init();

						PreferredLODLevel = FMath::Min(PreferredLODLevel, Emitter->LODLevels.Num());
						bSetLodLevels |= !bIsFirstCreate;//Only set lod levels if we init any instances and it's not the first creation time.
					}
				}
				else
				{
					if (Instance)
					{
#if STATS
						Instance->PreDestructorCall();
#endif
						delete Instance;
						EmitterInstances[Idx] = NULL;
						bClearDynamicData = true;
					}
				}
			}
		}

		if (bClearDynamicData)
		{
			ClearDynamicData();
		}

		if (bSetLodLevels)
		{
			if (PreferredLODLevel != LODLevel)
			{
				// This should never be higher...
				check(PreferredLODLevel < LODLevel);
				LODLevel = PreferredLODLevel;
			}

			for (int32 Idx = 0; Idx < EmitterInstances.Num(); Idx++)
			{
				FParticleEmitterInstance* Instance = EmitterInstances[Idx];
				// set the LOD levels here
				if (Instance)
				{
					Instance->CurrentLODLevelIndex	= LODLevel;

					// small safety net for OR-11322; can be removed if the ensure never fires after the change in SetTemplate (reset all instances LOD indices to 0)
					if (Instance->CurrentLODLevelIndex >= Instance->SpriteTemplate->LODLevels.Num())
					{
						Instance->CurrentLODLevelIndex = Instance->SpriteTemplate->LODLevels.Num()-1;
						ensureMsgf(false, TEXT("LOD access out of bounds (OR-11322). Please let olaf.piesche or simon.tovey know."));
					}
					Instance->CurrentLODLevel = Instance->SpriteTemplate->LODLevels[Instance->CurrentLODLevelIndex];
				}
			}
		}
	}

}

void UParticleSystemComponent::ResetParticles(bool bEmptyInstances)
{
	ForceAsyncWorkCompletion(ENSURE_AND_STALL);
	UE_LOGF(LogParticles,Verbose,
		"ResetParticles @ %fs %ls bEmptyInstances=%ls", GetWorld() ? GetWorld()->TimeSeconds : 0.0f,
		Template != NULL ? *Template->GetName() : TEXT("NULL"), bEmptyInstances ? TEXT("true") : TEXT("false"));

	UWorld* OwningWorld = GetWorld();

	//Also consider this deactivation.
	if (IsActive())
	{
		OnSystemPreActivationChange.Broadcast(this, false);
	}

	const bool bIsGameWorld = OwningWorld ? OwningWorld->IsGameWorld() : !GIsEditor;

	// Remove instances from scene.
	for( int32 InstanceIndex=0; InstanceIndex<EmitterInstances.Num(); InstanceIndex++ )
	{
		FParticleEmitterInstance* EmitterInstance = EmitterInstances[InstanceIndex];
		if( EmitterInstance )
		{
			if (GbEnableGameThreadLODCalculation == false)
			{
				if (!(!bIsGameWorld || bEmptyInstances))
				{
					EmitterInstance->SpriteTemplate	= NULL;
				}
			}
		}
	}

	// Set the system as inactive
	SetActiveFlag(false);

	// Remove instances if we're not running gameplay.ww
	if (!bIsGameWorld || bEmptyInstances)
	{
		for (int32 EmitterIndex = 0; EmitterIndex < EmitterInstances.Num(); EmitterIndex++)
		{
			FParticleEmitterInstance* EmitInst = EmitterInstances[EmitterIndex];
			if (EmitInst)
			{
#if STATS
				EmitInst->PreDestructorCall();
#endif
				delete EmitInst;
				EmitterInstances[EmitterIndex] = NULL;
			}
		}
		EmitterInstances.Empty();
		ClearDynamicData();
	}
	else
	{
		for (int32 EmitterIndex = 0; EmitterIndex < EmitterInstances.Num(); EmitterIndex++)
		{
			FParticleEmitterInstance* EmitInst = EmitterInstances[EmitterIndex];
			if (EmitInst)
			{
				EmitInst->Rewind();
			}
		}
	}

	// Mark render state dirty to deregister the component with the scene.
	MarkRenderStateDirty();
}

void UParticleSystemComponent::ResetBurstLists()
{
	ForceAsyncWorkCompletion(STALL);
	for (int32 i=0; i<EmitterInstances.Num(); i++)
	{
		if (EmitterInstances[i])
		{
			EmitterInstances[i]->ResetBurstList();
		}
	}
}

void UParticleSystemComponent::SetTemplate(class UParticleSystem* NewTemplate)
{
	SCOPE_CYCLE_COUNTER(STAT_ParticleSetTemplateTime);
	ForceAsyncWorkCompletion(STALL);

	if (PoolingMethod != EPSCPoolMethod::None)
	{
		UE_LOGF(LogParticles, Warning, "Changing template on pooled PSC! This will cause a reinit of the system, eliminating the benefits of pooling! Please avoid doing this.\nPSC: %ls\nOld Template: %ls\nNew Template: %ls"
		, *GetFullName(), *GetFullNameSafe(Template), *GetFullNameSafe(NewTemplate));
	}

	if( GIsAllowingParticles || GIsEditor ) 
	{
		bIsViewRelevanceDirty = true;

		bool bIsTemplate = IsTemplate();
		bWasCompleted = false;
		// remember if we were active and therefore should restart after setting up the new template
		bWasActive = IsActive() && !bWasDeactivated; 
		bool bResetInstances = false;
		if (NewTemplate != Template)
		{
			bIsElligibleForAsyncTick = false;
			bIsElligibleForAsyncTickComputed = false;
			bResetInstances = true;
		}
		if (bIsTemplate == false)
		{
			ResetParticles(bResetInstances);
		}

		Template = NewTemplate;
		if (Template)
		{
			WarmupTime = Template->WarmupTime;
		}
		else
		{
			WarmupTime = 0.0f;
		}

		// set the LOD level to 0 in case we're recycling the component, so InitParticles doesn't mistakenly grab an invalid LOD level
		// speculative fix for OR-11322. May become permanent if the ensure in InitParticles never fires.
		LODLevel = 0;

		SetComponentTickEnabled(false);

		if(NewTemplate && IsRegistered())
		{
			if ((bAutoActivate || bWasActive) && (bIsTemplate == false))
			{
				ActivateSystem();
			}
			else
			{
				InitializeSystem();
			}

			if (!SceneProxy || bResetInstances)
			{
				MarkRenderStateDirty();
			}
		}
	}
	else
	{
		Template = NULL;
	}
	if (!ensureMsgf(IsRenderStateDirty() || EmitterMaterials.Num() == 0, TEXT("About to lose material references without calling MarkRenderStateDirty on: %s"), *GetOwner()->GetName()))
	{
		MarkRenderStateDirty();
	}
	
	EmitterMaterials.Empty();

	for (int32 Idx = 0; Idx < EmitterInstances.Num(); Idx++)
	{
		FParticleEmitterInstance* Instance = EmitterInstances[Idx];
		// set the LOD levels to 0 in case we're recycling the component, so InitParticles doesn't mistakenly grab an invalid LOD level
		if (Instance)
		{
			Instance->CurrentLODLevelIndex = 0;
		}
	}

	if (ShouldBeTickManaged())
	{
		PrimaryComponentTick.bStartWithTickEnabled = false;
	}
}

void UParticleSystemComponent::ActivateSystem(bool bFlagAsJustAttached)
{
	CSV_SCOPED_TIMING_STAT_EXCLUSIVE(Effects);
	SCOPE_CYCLE_COUNTER(STAT_ParticleActivateTime);
	SCOPE_CYCLE_COUNTER(STAT_ParticlesOverview_GT);
	PARTICLE_PERF_STAT_CYCLES_GT(FParticlePerfStatsContext(GetWorld(), Template, this), Activation);
	ForceAsyncWorkCompletion(STALL);

	if (IsTemplate() == true || !IsRegistered() || 	!FApp::CanEverRender())
	{
		return;
	}

	if (!CascadeLocal::AllowTemplate(Template))
	{
		Template = nullptr;
	}

#if WITH_STATE_STREAM_ACTOR
	if (UseParticleSystemStateStream)
	{
		FParticleSystemStaticState Ss;
		FParticleSystemDynamicState Ds;
		Ds.SetSystemAsset(Template);
		Ds.SetTransform(GetOrCreateTransformHandle());
		ParticleSystemHandle = GetWorld()->GetStateStream<IParticleSystemStateStream>().Game_CreateInstance(Ss, Ds);
		return;
	}
#endif

	bOldPositionValid = false;
	OldPosition = FVector::ZeroVector;
	PartSysVelocity = FVector::ZeroVector;
	
	// Set tile for LWC offset
	LWCTile = FLargeWorldRenderScalar::GetTileFor(GetComponentLocation());

	UWorld* World = GetWorld();
	check(World);
	UE_LOGF(LogParticles,Verbose,
		"ActivateSystem @ %fs %ls", World->TimeSeconds,
		Template != NULL ? *Template->GetName() : TEXT("NULL"));

	const bool bIsGameWorld = World->IsGameWorld();

	if (UE_LOG_ACTIVE(LogParticles,VeryVerbose))
	{
		if (Template)
		{
			if (EmitterInstances.Num() > 0)
			{
				int32 LiveCount = 0;

				for (int32 EmitterIndex =0; EmitterIndex < EmitterInstances.Num(); EmitterIndex++)
				{
					FParticleEmitterInstance* EmitInst = EmitterInstances[EmitterIndex];
					if (EmitInst)
					{
						LiveCount += EmitInst->ActiveParticles;
					}
				}

				if (LiveCount > 0)
				{
					UE_LOGF(LogParticles, Log, "ActivateSystem called on PSysComp w/ live particles - %5d, %ls",
						LiveCount, *(Template->GetFullName()));
				}
			}
		}
	}

	// System settings may have been lowered. Support late deactivation.
	const bool bDetailModeAllowsRendering = DetailMode <= GetCurrentDetailMode();

	if( GIsAllowingParticles && bDetailModeAllowsRendering && Template )
	{
		// Auto attach if requested
		const bool bWasAutoAttached = bDidAutoAttach;
		bDidAutoAttach = false;
		if (bAutoManageAttachment && bIsGameWorld)
		{
			USceneComponent* NewParent = AutoAttachParent.Get();
			if (NewParent)
			{
				const bool bAlreadyAttached = GetAttachParent() && (GetAttachParent() == NewParent) && (GetAttachSocketName() == AutoAttachSocketName) && GetAttachParent()->GetAttachChildren().Contains(this);
				if (!bAlreadyAttached)
				{
					bDidAutoAttach = bWasAutoAttached;
					CancelAutoAttachment(true, World);
					SavedAutoAttachRelativeLocation = GetRelativeLocation();
					SavedAutoAttachRelativeRotation = GetRelativeRotation();
					SavedAutoAttachRelativeScale3D = GetRelativeScale3D();
					AttachToComponent(NewParent, FAttachmentTransformRules(AutoAttachLocationRule, AutoAttachRotationRule, AutoAttachScaleRule, bAutoAttachWeldSimulatedBodies), AutoAttachSocketName);
				}

				bDidAutoAttach = true;
				bFlagAsJustAttached = true;
			}
			else
			{
				CancelAutoAttachment(true, World);
			}
		}

		AccumTickTime = 0.0;

		if (!IsActive())
		{
			LastSignificantTime = World->GetTimeSeconds();
			RequiredSignificance = EParticleSignificanceLevel::Low;

		//Call this now after any attachment has happened.
			OnSystemPreActivationChange.Broadcast(this, true);
		}

		//We start this here as before the PreActivation call above, we don't know if this component is managing significance or not.
		FInGameScopedCycleCounter InGameCycleCounter(World, EInGamePerfTrackers::VFXSignificance, EInGamePerfTrackerThreads::GameThread, bIsManagingSignificance);

		if (bFlagAsJustAttached)
		{
			bJustRegistered = true;
		}
	
		// Stop suppressing particle spawning.
		bSuppressSpawning = false;
		
		// Set the system as active
		bool bNeedToUpdateTransform = bWasDeactivated;
		bWasCompleted = false;
		bWasDeactivated = false;
		SetActiveFlag(true);
		bWasActive = false; // Set to false now, it may get set to true when it's deactivated due to unregister
		SetComponentTickEnabled(true);

		// Force an LOD update - [op] do this before InitializeSystem, as that's going to set LOD level on all instances 
		if ((bIsGameWorld || (GIsEditor && GEngine->bEnableEditorPSysRealtimeLOD)) && (GbEnableGameThreadLODCalculation == true))
		{
			FVector EffectPosition = GetComponentLocation();
			int32 DesiredLODLevel = DetermineLODLevelForLocation(EffectPosition);
			SetLODLevel(DesiredLODLevel);
		}
		else
		{
			bForceLODUpdateFromRenderer = true;
		}


		// if no instances, or recycling
		if (EmitterInstances.Num() == 0 || (bIsGameWorld && (!bAutoActivate || bHasBeenActivated)))
		{
			InitializeSystem();
		}
		else if (EmitterInstances.Num() > 0 && !bIsGameWorld)
		{
			// If currently running, re-activating rewinds the emitter to the start. Existing particles should stick around.
			for (int32 i=0; i<EmitterInstances.Num(); i++)
			{
				if (EmitterInstances[i])
				{
					EmitterInstances[i]->Rewind();
					EmitterInstances[i]->SetHaltSpawning(false);
					EmitterInstances[i]->SetHaltSpawningExternal(false);
				}
			}
		}




		// Flag the system as having been activated at least once
		bHasBeenActivated = true;


		// Clear tick time
		TimeSinceLastTick = 0;

		int32 DesiredLODLevel = 0;
		bool bCalculateLODLevel = 
			(bOverrideLODMethod == true) ? (LODMethod != PARTICLESYSTEMLODMETHOD_DirectSet) : 
				(Template ? (Template->LODMethod != PARTICLESYSTEMLODMETHOD_DirectSet) : false);

		if (bCalculateLODLevel)
		{
			FVector EffectPosition = GetComponentLocation();
			DesiredLODLevel = DetermineLODLevelForLocation(EffectPosition);
			if (GbEnableGameThreadLODCalculation == true)
			{
				if (DesiredLODLevel != LODLevel)
				{
					SetActiveFlag(true);
					SetComponentTickEnabled(true);
				}
				SetLODLevel(DesiredLODLevel);
			}
		}

		if (WarmupTime != 0.0f)
		{
			bool bSaveSkipUpdate = bSkipUpdateDynamicDataDuringTick;
			bSkipUpdateDynamicDataDuringTick = true;
			bWarmingUp = true;
			ResetBurstLists();

			float WarmupElapsed = 0.f;
			float WarmupTimestep = 0.032f;
			if (WarmupTickRate > 0)
			{
				WarmupTimestep = (WarmupTickRate <= WarmupTime) ? WarmupTickRate : WarmupTime;
			}

			while (WarmupElapsed < WarmupTime)
			{
				TickComponent(WarmupTimestep, LEVELTICK_All, NULL);
				WarmupElapsed += WarmupTimestep;
			}

			bWarmingUp = false;
			WarmupTime = 0.0f;
			bSkipUpdateDynamicDataDuringTick = bSaveSkipUpdate;
		}

		//We are definitely insignificant already so set insignificant before we ever begin ticking.
		if (!bIsDuringRegister && bIsManagingSignificance && Template->GetHighestSignificance() < RequiredSignificance && Template->InsignificanceDelay == 0.0f)
		{
			OnSignificanceChanged(false, true);
		}
	}

	// Mark render state dirty to ensure the scene proxy is added and registered with the scene.
	MarkRenderStateDirty();

	World = GetWorld(); // refresh the world pointer as it may have changed by this point
	if(!bWasDeactivated && !bWasCompleted && ensure(World))
	{
		SetLastRenderTime(World->GetTimeSeconds());
	}
}

void UParticleSystemComponent::Complete()
{
	UWorld* World = GetWorld();
	check(World);

	UE_LOGF(LogParticles, Verbose,
		"HasCompleted()==true @ %fs %ls", GetWorld()->TimeSeconds,
		Template != NULL ? *Template->GetName() : TEXT("NULL"));

	OnSystemFinished.Broadcast(this);

	// When system is done - destroy all subemitters etc. We don't need them any more.
	ResetParticles();
	SetActiveFlag(false);
	SetComponentTickEnabled(false);

	if (PoolingMethod == EPSCPoolMethod::AutoRelease)
	{
		World->GetPSCPool().ReclaimWorldParticleSystem(this);
	}
	else if (PoolingMethod == EPSCPoolMethod::ManualRelease_OnComplete)
	{
		PoolingMethod = EPSCPoolMethod::ManualRelease;
		World->GetPSCPool().ReclaimWorldParticleSystem(this);
	}
	else if (bAutoDestroy)
	{
		DestroyComponent();
	}
	else if (bAutoManageAttachment)
	{
		CancelAutoAttachment(/*bDetachFromParent=*/ true, World);
	}
}

void UParticleSystemComponent::DeactivateSystem()
{
	UWorld* World = GetWorld();
	FInGameScopedCycleCounter InGameCycleCounter(World, EInGamePerfTrackers::VFXSignificance, EInGamePerfTrackerThreads::GameThread, bIsManagingSignificance);
	SCOPE_CYCLE_COUNTER(STAT_ParticlesOverview_GT);

#if WITH_STATE_STREAM_ACTOR
	if (UseParticleSystemStateStream)
	{
		ParticleSystemHandle = {};
		return;
	}
#endif

	if (IsTemplate() == true)
	{
		return;
	}
	ForceAsyncWorkCompletion(STALL);

	check(World);

	//We have seen some edge case where the world can be null here so avoid the crash and try to leave the component in a decent state until we can fix the underlying issue.
	if (World == nullptr)
	{
		UE_LOGF(LogParticles, Error, "DeactivateSystem called on PSC with null World ptr! %ls", Template != NULL ? *Template->GetName() : TEXT("NULL"));

		ResetParticles(true);
		bDeactivateTriggered = false;
		bSuppressSpawning = true;
		bWasDeactivated = true;
		return;
	}

	UE_LOGF(LogParticles, Verbose,
		"DeactivateSystem @ %fs %ls", World->TimeSeconds,
		Template != NULL ? *Template->GetName() : TEXT("NULL"));

	if (IsActive())
	{
		OnSystemPreActivationChange.Broadcast(this, false);
	}

	bDeactivateTriggered = false;
	bSuppressSpawning = true;
	bWasDeactivated = true;

	bool bShouldMarkRenderStateDirty = false;
	for (int32 i = 0; i < EmitterInstances.Num(); i++)
	{
		FParticleEmitterInstance*	Instance = EmitterInstances[i];
		if (Instance)
		{
			if (Instance->bKillOnDeactivate)
			{
				//UE_LOGF(LogParticles, Log, "%ls killed on deactivate",EmitterInstances(i)->GetName());
#if STATS
				Instance->PreDestructorCall();
#endif
				// clean up other instances that may point to this one
				for (int32 InnerIndex=0; InnerIndex < EmitterInstances.Num(); InnerIndex++)
				{
					if (InnerIndex != i && EmitterInstances[InnerIndex] != NULL)
					{
						EmitterInstances[InnerIndex]->OnEmitterInstanceKilled(Instance);
					}
				}
				delete Instance;
				EmitterInstances[i] = NULL;
				bShouldMarkRenderStateDirty = true;
			}
			else
			{
				Instance->OnDeactivateSystem();
			}
		}
	}

	if (bShouldMarkRenderStateDirty)
	{
		ClearDynamicData();
		MarkRenderStateDirty();
	}

	//We have to ensure ticking is enabled so that this component completes and is can be destroyed etc correctly.
	//TODO: What if there are immortal particles but bKillOnDeactivate is false? Need to mark emitters with currently immortal particles, kill them and warn the user.
	SetComponentTickEnabled(true);

	SetLastRenderTime(World->GetTimeSeconds());
}

void UParticleSystemComponent::CancelAutoAttachment(bool bDetachFromParent, const UWorld* MyWorld)
{
	if (bAutoManageAttachment && MyWorld && MyWorld->IsGameWorld())
	{
		if (bDidAutoAttach)
		{
			// Restore relative transform from before attachment. Actual transform will be updated as part of DetachFromParent().
			SetRelativeLocation_Direct(SavedAutoAttachRelativeLocation);
			SetRelativeRotation_Direct(SavedAutoAttachRelativeRotation);
			SetRelativeScale3D_Direct(SavedAutoAttachRelativeScale3D);
			bDidAutoAttach = false;
		}

		if (bDetachFromParent)
		{
			UWorld* World = GetWorld();
			if (!World || World->IsGameWorld())
			{
				DetachFromComponent(FDetachmentTransformRules(EDetachmentRule::KeepRelative, /*bCallModify=*/ false));
			}
		}
	}
}

bool UParticleSystemComponent::ShouldBeTickManaged()const
{
#if WITH_EDITOR
	if (!Editor_CanBeTickManaged())
	{
		return false;
	}
#endif
	return
		GbEnablePSCWorldManager &&
		Template && Template->AllowManagedTicking() &&
		PrimaryComponentTick.GetPrerequisites().Num() <= 1 &&//Don't batch tick if we have complex prerequisites.
		GetAttachChildren().Num() == 0 &&//Don't batch tick if people are attached and dependent on us.
		!IsNetMode(NM_DedicatedServer);//Never allow for dedicated servers. Use existing tick mechanisms to avoid these.
}

void UParticleSystemComponent::ComputeCanTickInAnyThread()
{
	check(!bIsElligibleForAsyncTickComputed);
	bIsElligibleForAsyncTick = false;
	if (Template)
	{
		bIsElligibleForAsyncTickComputed = true;
		bIsElligibleForAsyncTick = Template->CanTickInAnyThread();
	}
}

bool UParticleSystemComponent::ShouldActivate() const
{
	return (Super::ShouldActivate() || (bWasDeactivated || bWasCompleted));
}

void UParticleSystemComponent::Activate(bool bReset) 
{
	// If the particle system can't ever render (ie on dedicated server or in a commandlet) than do not activate...
	// Occasionaly we can arrive here with no template so check that here too.
	if (FApp::CanEverRender() && Template)
	{
		//Clear any pending deactivation.
		bDeactivateTriggered = false;

		if (bReset || ShouldActivate()==true)
		{
			ActivateSystem(bReset);

			if (IsActive())
			{
				OnComponentActivated.Broadcast(this, bReset);
			}
		}
	}
}

int32 GbDeferrPSCDeactivation = 0;
FAutoConsoleVariableRef CVarDeferrPSCDeactivation(
	TEXT("fx.DeferrPSCDeactivation"),
	GbDeferrPSCDeactivation,
	TEXT("If > 0, all deactivations on Particle System Components is deferred until next tick."),
	ECVF_Scalability
);

void UParticleSystemComponent::Deactivate()
{
	if (ShouldActivate()==false)
	{
		if (GbDeferrPSCDeactivation)
		{
			DeactivaateNextTick();
		}
		else
		{
			//UE_LOGF(LogParticles, Warning, "Deactivate() 0x%p - %ls - %ls", this, Template ? *Template->GetFullName() : TEXT("null template"), *GetFullName());
			DeactivateSystem();

			if (bWasDeactivated)
			{
				OnComponentDeactivated.Broadcast(this);
			}
		}		
	}
}

void UParticleSystemComponent::DeactivateImmediate()
{
	Complete();
}

void UParticleSystemComponent::ApplyWorldOffset(const FVector& InOffset, bool bWorldShift)
{
	Super::ApplyWorldOffset(InOffset, bWorldShift);

	// Trigger a reset as the offset applying below does not work correctly with all emitter types
	// Niagara also resets so having Cascade follow the same path makes it consistent also
	bResetTriggered = true;

#if 0
	OldPosition += InOffset;
	
	for (auto It = EmitterInstances.CreateIterator(); It; ++It)
	{
		FParticleEmitterInstance* EmitterInstance = *It;
		if (EmitterInstance != NULL)
		{
			EmitterInstance->ApplyWorldOffset(InOffset, bWorldShift);
		}
	}
#endif
}

void UParticleSystemComponent::ResetToDefaults()
{
	ForceAsyncWorkCompletion(STALL);
	if (!IsTemplate())
	{
		// make sure we're fully stopped and unregistered
		DeactivateSystem();
		SetTemplate(NULL);

		if(IsRegistered())
		{
			UnregisterComponent();
		}

		UParticleSystemComponent* Default = (UParticleSystemComponent*)(GetArchetype());

		// copy all non-native, non-duplicatetransient, non-Component properties we have from all classes up to and including UActorComponent
		for (FProperty* Property = GetClass()->PropertyLink; Property != NULL; Property = Property->PropertyLinkNext)
		{
			if ( !(Property->PropertyFlags & CPF_DuplicateTransient) && !(Property->PropertyFlags & (CPF_InstancedReference | CPF_ContainsInstancedReference)) &&
				Property->GetOwnerClass()->IsChildOf(UActorComponent::StaticClass()) )
			{
				Property->CopyCompleteValue_InContainer(this, Default);
			}
		}
	}
}

void UParticleSystemComponent::UpdateInstances(bool bEmptyInstances)
{
	if (GIsEditor && IsRegistered())
	{
		ForceAsyncWorkCompletion(STALL);
		ResetParticles(bEmptyInstances);

		InitializeSystem();
		if (bAutoActivate)
		{
			ActivateSystem();
		}

		if (Template && Template->bUseFixedRelativeBoundingBox)
		{
			UpdateComponentToWorld();
		}
	}
}

int32 UParticleSystemComponent::GetNumActiveParticles()const
{
	ForceAsyncWorkCompletion(STALL);
	int32 NumParticles = 0;
	for (int32 i=0; i<EmitterInstances.Num(); i++)
	{
		FParticleEmitterInstance* Instance = EmitterInstances[i];

		if (Instance != NULL)
		{
			NumParticles += Instance->ActiveParticles;
		}
	}
	return NumParticles;
}

void UParticleSystemComponent::GetOwnedTrailEmitters(UParticleSystemComponent::TrailEmitterArray& OutTrailEmitters, const void* InOwner, bool bSetOwner)
{
	for (FParticleEmitterInstance* Inst : EmitterInstances)
	{
		if (Inst && Inst->IsTrailEmitter())
		{
			FParticleAnimTrailEmitterInstance* TrailEmitter = (FParticleAnimTrailEmitterInstance*)Inst;
			if (bSetOwner)
			{
				TrailEmitter->Owner = InOwner;
				OutTrailEmitters.Add(TrailEmitter);
			}
			else if (TrailEmitter->Owner == InOwner)
			{
				OutTrailEmitters.Add(TrailEmitter);
			}
		}
	}
}

void UParticleSystemComponent::BeginTrails(FName InFirstSocketName, FName InSecondSocketName, ETrailWidthMode InWidthMode, float InWidth)
{
	ActivateSystem(true);
	for (FParticleEmitterInstance* Inst : EmitterInstances)
	{
		if (Inst)
		{
			Inst->BeginTrail();
			Inst->SetTrailSourceData(InFirstSocketName, InSecondSocketName, InWidthMode, InWidth);
		}
	}
}

void UParticleSystemComponent::EndTrails()
{
	for (FParticleEmitterInstance* Inst : EmitterInstances)
	{
		if (Inst)
		{
			Inst->EndTrail();
		}
	}
	DeactivateSystem();
}

void UParticleSystemComponent::SetTrailSourceData(FName InFirstSocketName, FName InSecondSocketName, ETrailWidthMode InWidthMode, float InWidth)
{
	for (FParticleEmitterInstance* Inst : EmitterInstances)
	{
		if (Inst)
		{
			Inst->SetTrailSourceData(InFirstSocketName, InSecondSocketName, InWidthMode, InWidth);
		}
	}
}

void UParticleSystemComponent::ReleaseToPool()
{
	if (PoolingMethod != EPSCPoolMethod::ManualRelease)
	{
		UE_LOGF(LogParticles, Warning, "Manually releasing a PSC to the pool that was not spawned with EPSCPoolMethod::ManualRelease. Template=%ls Component=%ls",
			Template ? *Template->GetPathName() : TEXT("NULL"),
			*GetPathName()
		);
		return;
	}
	
	if (bWasCompleted)
	{
		//If we're already complete then release to the pool straight away.
		UWorld* World = GetWorld();
		check(World);
		World->GetPSCPool().ReclaimWorldParticleSystem(this);
	}
	else
	{
		//If we haven't completed, deactivate and defer release to pool.
		PoolingMethod = EPSCPoolMethod::ManualRelease_OnComplete;
		Deactivate();
	}
}

bool UParticleSystemComponent::HasCompleted()
{
	ForceAsyncWorkCompletion(STALL);
	bool bHasCompleted = true;
	bool bCanBeDeactivated = true;

	// If we're currently capturing or replaying captured frames, then we'll stay active for that
	if( ReplayState != PRS_Disabled )
	{
		// While capturing, we want to stay active so that we'll just record empty frame data for
		// completed particle systems.  While replaying, we never want our particles/meshes removed from
		// the scene, so we'll force the system to stay alive!
		return false;
	}

	bool bClearDynamicData = false;
	for (int32 i=0; i<EmitterInstances.Num(); i++)
	{
		FParticleEmitterInstance* Instance = EmitterInstances[i];

		if (Instance && Instance->CurrentLODLevel && Instance->bEnabled)
		{
			if (!Instance->bEmitterIsDone)
			{
				bCanBeDeactivated = false;
			}

			if (Instance->CurrentLODLevel->bEnabled)
			{
				if (Instance->CurrentLODLevel->RequiredModule->EmitterLoops > 0 || Instance->IsTrailEmitter())
				{
					if (bWasDeactivated && bSuppressSpawning)
					{
						if (Instance->ActiveParticles != 0)
						{
							bHasCompleted = false;
						}
					}
					else
					{
						if (Instance->HasCompleted())
						{
							if (Instance->bKillOnCompleted)
							{
#if STATS
								Instance->PreDestructorCall();
#endif
								// clean up other instances that may point to this one
								for (int32 InnerIndex=0; InnerIndex < EmitterInstances.Num(); InnerIndex++)
								{
									if (InnerIndex != i && EmitterInstances[InnerIndex] != NULL)
									{
										EmitterInstances[InnerIndex]->OnEmitterInstanceKilled(Instance);
									}
								}
								delete Instance;
								EmitterInstances[i] = NULL;
								bClearDynamicData = true;
							}
						}
						else
						{
							bHasCompleted = false;
						}
					}
				}
				else
				{
					if (bWasDeactivated)
					{
						if (Instance->ActiveParticles != 0)
						{
							bHasCompleted = false;
						}
					}
					else
					{
						bHasCompleted = false;
					}
				}
			}
			else
			{
				UParticleEmitter* Em = CastChecked<UParticleEmitter>(Instance->CurrentLODLevel->GetOuter());
				if (Em && Em->bDisabledLODsKeepEmitterAlive)
				{
					bHasCompleted = false;
				}				
			}
		}
	}

	if (bCanBeDeactivated && Template && Template->bAutoDeactivate)
	{
		DeactivateSystem();
	}

	if (bClearDynamicData)
	{
		ClearDynamicData();
	}
	
	return bHasCompleted;
}

void UParticleSystemComponent::InitializeSystem()
{
	SCOPE_CYCLE_COUNTER(STAT_ParticleInitializeTime);
	ForceAsyncWorkCompletion(STALL);

	if (!IsRegistered() || (FXSystem == NULL))
	{
		// Don't warn in a commandlet, we're expected not to have a scene / FX system.
		if (!IsRunningCommandlet() && !IsRunningDedicatedServer())
		{
			// We're also not expected to have a scene / FX system when we belong to an inactive world.
			UWorld* OwnerWorld = GetWorld();
			if (!OwnerWorld || OwnerWorld->WorldType != EWorldType::Inactive)
			{
				UE_LOGF(LogParticles,Warning,"InitializeSystem called on an unregistered component. Template=%ls Component=%ls",
					Template ? *Template->GetPathName() : TEXT("NULL"),
					*GetPathName()
					);
			}
		}
		return;
	}

	// At this point the component must be associated with an FX system.
	check( FXSystem != NULL );

	check(GetWorld());
	UE_LOGF(LogParticles,Verbose,
		"InitializeSystem @ %fs %ls Component=0x%p FXSystem=0x%p", GetWorld()->TimeSeconds,
		Template != NULL ? *Template->GetName() : TEXT("NULL"), this, FXSystem);

	// System settings may have been lowered. Support late deactivation.
	const bool bDetailModeAllowsRendering = DetailMode <= GetCurrentDetailMode();

	if( GIsAllowingParticles && bDetailModeAllowsRendering )
	{
		if (IsTemplate() == true)
		{
			return;
		}

		if( Template != NULL )
		{
			EmitterDelay = Template->Delay;

			if( Template->bUseDelayRange )
			{
				const float	Rand = RandomStream.FRand();
				EmitterDelay	 = Template->DelayLow + ((Template->Delay - Template->DelayLow) * Rand);
			}
		}

		// Allocate the emitter instances and particle data
		InitParticles();
		if (IsRegistered())
		{
			AccumTickTime = 0.0;
			if ((IsActive() == false) && (bAutoActivate == true) && (bWasDeactivated == false))
			{
				SetActive(true);
			}
		}
	}
}


FString UParticleSystemComponent::GetDetailedInfoInternal() const
{
	FString Result;  

	if( Template != NULL )
	{
		Result = Template->GetPathName( NULL );
	}
	else
	{
		Result = TEXT("No_ParticleSystem");
	}

	return Result;  
}




void UParticleSystemComponent::ConditionalCacheViewRelevanceFlags(class UParticleSystem* NewTemplate)
{
	ForceAsyncWorkCompletion(ENSURE_AND_STALL);
	if (NewTemplate && (NewTemplate != Template))
	{
		bIsViewRelevanceDirty = true;
	}

	if (bIsViewRelevanceDirty)
	{
		UParticleSystem* TemplateToCache = Template;
		if (NewTemplate)
		{
			TemplateToCache = NewTemplate;
		}
		CacheViewRelevanceFlags(TemplateToCache);
		MarkRenderStateDirty();
	}
}

void UParticleSystemComponent::CacheViewRelevanceFlags(UParticleSystem* TemplateToCache)
{
	ForceAsyncWorkCompletion(ENSURE_AND_STALL);
	CachedViewRelevanceFlags.Empty();

	if (TemplateToCache)
	{
		for (int32 EmitterIndex = 0; EmitterIndex < TemplateToCache->Emitters.Num(); EmitterIndex++)
		{
			UParticleSpriteEmitter* Emitter = Cast<UParticleSpriteEmitter>(TemplateToCache->Emitters[EmitterIndex]);
			if (Emitter == NULL)
			{
				// Handle possible empty slots in the emitter array.
				continue;
			}
			FParticleEmitterInstance* EmitterInst = NULL;
			if (EmitterIndex < EmitterInstances.Num())
			{
				EmitterInst = EmitterInstances[EmitterIndex];
			}

			//@TODO I suspect this function can get called before emitter instances are created. That is bad and should be fixed up.
			if ( EmitterInst == NULL )
			{
				continue;
			}

			for (int32 LODIndex = 0; LODIndex < Emitter->LODLevels.Num(); LODIndex++)
			{
				UParticleLODLevel* EmitterLODLevel = Emitter->LODLevels[LODIndex];

				// Prime the array
				// This code assumes that the particle system emitters all have the same number of LODLevels. 
				if (LODIndex >= CachedViewRelevanceFlags.Num())
				{
					CachedViewRelevanceFlags.AddZeroed(1);
				}
				FMaterialRelevance& LODViewRel = CachedViewRelevanceFlags[LODIndex];
				check(EmitterLODLevel->RequiredModule);

				if (EmitterLODLevel->bEnabled == true)
				{
					auto World = GetWorld();
					EmitterInst->GatherMaterialRelevance(&LODViewRel, EmitterLODLevel, GetFeatureLevelShaderPlatform_Checked(World ? World->GetFeatureLevel() : GMaxRHIFeatureLevel));
				}
			}
		}
	}
	bIsViewRelevanceDirty = false;
}

void UParticleSystemComponent::RewindEmitterInstances()
{
	ForceAsyncWorkCompletion(STALL);
	for (int32 EmitterIndex = 0; EmitterIndex < EmitterInstances.Num(); EmitterIndex++)
	{
		FParticleEmitterInstance* EmitterInst = EmitterInstances[EmitterIndex];
		if (EmitterInst)
		{
			EmitterInst->Rewind();
		}
	}
}


void UParticleSystemComponent::SetBeamEndPoint(int32 EmitterIndex,FVector NewEndPoint)
{
	ForceAsyncWorkCompletion(STALL);
	if ((EmitterIndex >= 0) && (EmitterIndex < EmitterInstances.Num()))
	{
		FParticleEmitterInstance* EmitterInst = EmitterInstances[EmitterIndex];
		if (EmitterInst)
		{
			EmitterInst->SetBeamEndPoint(NewEndPoint);
		}
	}
}


void UParticleSystemComponent::SetBeamSourcePoint(int32 EmitterIndex,FVector NewSourcePoint,int32 SourceIndex)
{
	ForceAsyncWorkCompletion(STALL);
	if ((EmitterIndex >= 0) && (EmitterIndex < EmitterInstances.Num()))
	{
		FParticleEmitterInstance* EmitterInst = EmitterInstances[EmitterIndex];
		if (EmitterInst)
		{
			EmitterInst->SetBeamSourcePoint(NewSourcePoint, SourceIndex);
		}
	}
}


void UParticleSystemComponent::SetBeamSourceTangent(int32 EmitterIndex,FVector NewTangentPoint,int32 SourceIndex)
{
	ForceAsyncWorkCompletion(STALL);
	if ((EmitterIndex >= 0) && (EmitterIndex < EmitterInstances.Num()))
	{
		FParticleEmitterInstance* EmitterInst = EmitterInstances[EmitterIndex];
		if (EmitterInst)
		{
			EmitterInst->SetBeamSourceTangent(NewTangentPoint, SourceIndex);
		}
	}
}


void UParticleSystemComponent::SetBeamSourceStrength(int32 EmitterIndex,float NewSourceStrength,int32 SourceIndex)
{
	ForceAsyncWorkCompletion(STALL);
	if ((EmitterIndex >= 0) && (EmitterIndex < EmitterInstances.Num()))
	{
		FParticleEmitterInstance* EmitterInst = EmitterInstances[EmitterIndex];
		if (EmitterInst)
		{
			EmitterInst->SetBeamSourceStrength(NewSourceStrength, SourceIndex);
		}
	}
}


void UParticleSystemComponent::SetBeamTargetPoint(int32 EmitterIndex,FVector NewTargetPoint,int32 TargetIndex)
{
	ForceAsyncWorkCompletion(STALL);
	if ((EmitterIndex >= 0) && (EmitterIndex < EmitterInstances.Num()))
	{
		FParticleEmitterInstance* EmitterInst = EmitterInstances[EmitterIndex];
		if (EmitterInst)
		{
			EmitterInst->SetBeamTargetPoint(NewTargetPoint, TargetIndex);
		}
	}
}


void UParticleSystemComponent::SetBeamTargetTangent(int32 EmitterIndex,FVector NewTangentPoint,int32 TargetIndex)
{
	ForceAsyncWorkCompletion(STALL);
	if ((EmitterIndex >= 0) && (EmitterIndex < EmitterInstances.Num()))
	{
		FParticleEmitterInstance* EmitterInst = EmitterInstances[EmitterIndex];
		if (EmitterInst)
		{
			EmitterInst->SetBeamTargetTangent(NewTangentPoint, TargetIndex);
		}
	}
}

void UParticleSystemComponent::SetBeamTargetStrength(int32 EmitterIndex,float NewTargetStrength,int32 TargetIndex)
{
	ForceAsyncWorkCompletion(STALL);
	if ((EmitterIndex >= 0) && (EmitterIndex < EmitterInstances.Num()))
	{
		FParticleEmitterInstance* EmitterInst = EmitterInstances[EmitterIndex];
		if (EmitterInst)
		{
			EmitterInst->SetBeamTargetStrength(NewTargetStrength, TargetIndex);
		}
	}
}

bool UParticleSystemComponent::GetBeamEndPoint(int32 EmitterIndex, FVector& OutSourcePoint) const
{
	if ((EmitterIndex >= 0) && (EmitterIndex < EmitterInstances.Num()))
	{
		FParticleEmitterInstance* EmitterInst = EmitterInstances[EmitterIndex];
		if (EmitterInst)
		{
			return EmitterInst->GetBeamEndPoint(OutSourcePoint);
		}
	}

	return false;
}

bool UParticleSystemComponent::GetBeamSourcePoint(int32 EmitterIndex, int32 SourceIndex, FVector& OutSourcePoint) const
{
	if ((EmitterIndex >= 0) && (EmitterIndex < EmitterInstances.Num()))
	{
		FParticleEmitterInstance* EmitterInst = EmitterInstances[EmitterIndex];
		if (EmitterInst)
		{
			return EmitterInst->GetBeamSourcePoint(SourceIndex, OutSourcePoint);
		}
	}

	return false;
}

bool UParticleSystemComponent::GetBeamSourceTangent(int32 EmitterIndex, int32 SourceIndex, FVector& OutSourcePoint) const
{
	if ((EmitterIndex >= 0) && (EmitterIndex < EmitterInstances.Num()))
	{
		FParticleEmitterInstance* EmitterInst = EmitterInstances[EmitterIndex];
		if (EmitterInst)
		{
			return EmitterInst->GetBeamSourceTangent(SourceIndex, OutSourcePoint);
		}
	}

	return false;
}

bool UParticleSystemComponent::GetBeamSourceStrength(int32 EmitterIndex, int32 SourceIndex, float& OutSourceStrength) const
{
	if ((EmitterIndex >= 0) && (EmitterIndex < EmitterInstances.Num()))
	{
		FParticleEmitterInstance* EmitterInst = EmitterInstances[EmitterIndex];
		if (EmitterInst)
		{
			return EmitterInst->GetBeamSourceStrength(SourceIndex, OutSourceStrength);
		}
	}

	return false;
}
bool UParticleSystemComponent::GetBeamTargetPoint(int32 EmitterIndex, int32 TargetIndex, FVector& OutTargetPoint) const
{
	if ((EmitterIndex >= 0) && (EmitterIndex < EmitterInstances.Num()))
	{
		FParticleEmitterInstance* EmitterInst = EmitterInstances[EmitterIndex];
		if (EmitterInst)
		{
			return EmitterInst->GetBeamTargetPoint(TargetIndex, OutTargetPoint);
		}
	}

	return false;
}

bool UParticleSystemComponent::GetBeamTargetTangent(int32 EmitterIndex, int32 TargetIndex, FVector& OutTangentPoint) const
{
	if ((EmitterIndex >= 0) && (EmitterIndex < EmitterInstances.Num()))
	{
		FParticleEmitterInstance* EmitterInst = EmitterInstances[EmitterIndex];
		if (EmitterInst)
		{
			return EmitterInst->GetBeamTargetTangent(TargetIndex, OutTangentPoint);
		}
	}

	return false;
}

bool UParticleSystemComponent::GetBeamTargetStrength(int32 EmitterIndex, int32 TargetIndex, float& OutTargetStrength) const
{
	if ((EmitterIndex >= 0) && (EmitterIndex < EmitterInstances.Num()))
	{
		FParticleEmitterInstance* EmitterInst = EmitterInstances[EmitterIndex];
		if (EmitterInst)
		{
			return EmitterInst->GetBeamTargetStrength(TargetIndex, OutTargetStrength);
		}
	}

	return false;
}


void UParticleSystemComponent::SetEmitterEnable(FName EmitterName, bool bNewEnableState)
{
	ForceAsyncWorkCompletion(STALL);
	for (int32 EmitterIndex = 0; EmitterIndex < EmitterInstances.Num(); ++EmitterIndex)
	{
		FParticleEmitterInstance* EmitterInst = EmitterInstances[EmitterIndex];
		if (EmitterInst && EmitterInst->SpriteTemplate)
		{
			if (EmitterInst->SpriteTemplate->EmitterName == EmitterName)
			{
				EmitterInst->SetHaltSpawningExternal(!bNewEnableState);
			}
		}
	}
}

int32 UParticleSystemComponent::DetermineLODLevelForLocation(const FVector& EffectLocation)
{
	// No particle system, ignore
	if (Template == NULL)
	{
		return 0;
	}

	// Don't bother if we only have 1 LOD level... Or if we want to ignore distance comparisons.
	if (Template->LODDistances.Num() <= 1 || Template->LODMethod == PARTICLESYSTEMLODMETHOD_DirectSet)
	{
		return 0;
	}

	check(IsInGameThread());
	int32 Retval = 0;
	
	// Run this for all local player controllers.
	// If several are found (split screen?). Take the closest for highest LOD.
	UWorld* World = GetWorld();
	if(World != NULL)
	{
		TArray<FVector,TInlineAllocator<8> > PlayerViewLocations;
		if (World->GetPlayerControllerIterator())
		{
			for( FConstPlayerControllerIterator Iterator = World->GetPlayerControllerIterator(); Iterator; ++Iterator )
			{
				APlayerController* PlayerController = Iterator->Get();
				if(PlayerController && PlayerController->IsLocalPlayerController())
				{
					FVector* POVLoc = new(PlayerViewLocations) FVector;
					FRotator POVRotation;
					PlayerController->GetPlayerViewPoint(*POVLoc, POVRotation);
				}
			}
		}
		else
		{
			PlayerViewLocations.Append(World->ViewLocationsRenderedLastFrame);
		}

		// This will now put everything in LODLevel 0 (high detail) by default
		FVector::FReal LODDistanceSqr = (PlayerViewLocations.Num() ? FMath::Square(WORLD_MAX) : 0.0f);
		for (const FVector& ViewLocation : PlayerViewLocations)
		{
			const FVector::FReal DistanceToEffectSqr = FVector(ViewLocation - EffectLocation).SizeSquared();
			if (DistanceToEffectSqr < LODDistanceSqr)
			{
				LODDistanceSqr = DistanceToEffectSqr;
			}
		}

		// Find appropriate LOD based on distance
		Retval = Template->LODDistances.Num() - 1;
		for (int32 LODIdx = 1; LODIdx < Template->LODDistances.Num(); ++LODIdx)
		{
			if (LODDistanceSqr < FMath::Square(Template->LODDistances[LODIdx]))
			{
				Retval = LODIdx - 1;
				break;
			}
		}
	}

	return Retval;
}

void UParticleSystemComponent::SetLODLevel(int32 InLODLevel)
{
	ForceAsyncWorkCompletion(STALL);
	if (Template == NULL)
	{
		return;
	}

	if (Template->LODDistances.Num() == 0)
	{
		return;
	}

	int32 NewLODLevel = FMath::Clamp(InLODLevel + GParticleLODBias, 0, Template->GetLODLevelCount() - 1);
	if (LODLevel != NewLODLevel)
	{
		MarkRenderStateDirty();

		const int32 OldLODLevel = LODLevel;
		LODLevel = NewLODLevel;

		for (int32 i=0; i<EmitterInstances.Num(); i++)
		{
			FParticleEmitterInstance* Instance = EmitterInstances[i];
			if (Instance)
			{
				Instance->SetCurrentLODIndex(LODLevel, true);
			}
		}
	}
}

int32 UParticleSystemComponent::GetLODLevel()
{
	return LODLevel;
}
/**
 *	Set a named float instance parameter on this ParticleSystemComponent.
 *	Updates the parameter if it already exists, or creates a new entry if not.
	This maps a boolean to a float for parity as cascade doesn't have booleans.
	This for adding functionality to the parent UFXSystemComponent to set boolean
	variables.
 */
void UParticleSystemComponent::SetBoolParameter(FName Name, bool Value)
{
	SetFloatParameter(Name, (Value ? 1.0f : 0.0f));

}
/**
 *	Set a named float instance parameter on this ParticleSystemComponent.
 *	Updates the parameter if it already exists, or creates a new entry if not.
	This maps a int to a float for parity as cascade doesn't have booleans.
	This for adding functionality to the parent UFXSystemComponent to set int
	variables.
 */
void UParticleSystemComponent::SetIntParameter(FName Name, int Value)
{
	SetFloatParameter(Name, float(Value));
}

/** 
 *	Set a named float instance parameter on this ParticleSystemComponent. 
 *	Updates the parameter if it already exists, or creates a new entry if not. 
 */
void UParticleSystemComponent::SetFloatParameter(FName Name, float Param)
{
	LLM_SCOPE(ELLMTag::Particles);

	if(Name == NAME_None)
	{
		return;
	}
	check(IsInGameThread());

	// First see if an entry for this name already exists
	for (int32 i = 0; i < InstanceParameters.Num(); i++)
	{
		FParticleSysParam& P = InstanceParameters[i];
		if (P.Name == Name && P.ParamType == PSPT_Scalar)
		{
			P.Scalar = Param;
			return;
		}
	}

	// We didn't find one, so create a new one.
	int32 NewParamIndex = InstanceParameters.AddZeroed();
	InstanceParameters[NewParamIndex].Name = Name;
	InstanceParameters[NewParamIndex].ParamType = PSPT_Scalar;
	InstanceParameters[NewParamIndex].Scalar = Param;
}


void UParticleSystemComponent::SetFloatRandParameter(FName ParameterName,float Param,float ParamLow)
{
	LLM_SCOPE(ELLMTag::Particles);

	if (ParameterName == NAME_None)
	{
		return;
	}
	check(IsInGameThread());

	// First see if an entry for this name already exists
	for (int32 i = 0; i < InstanceParameters.Num(); i++)
	{
		FParticleSysParam& P = InstanceParameters[i];
		if (P.Name == ParameterName && P.ParamType == PSPT_ScalarRand)
		{
			P.Scalar = Param;
			P.Scalar_Low = ParamLow;
			return;
		}
	}

	// We didn't find one, so create a new one.
	int32 NewParamIndex = InstanceParameters.AddZeroed();
	InstanceParameters[NewParamIndex].Name = ParameterName;
	InstanceParameters[NewParamIndex].ParamType = PSPT_ScalarRand;
	InstanceParameters[NewParamIndex].Scalar = Param;
	InstanceParameters[NewParamIndex].Scalar_Low = ParamLow;
}


void UParticleSystemComponent::SetVectorParameter(FName Name, FVector Param)
{
	LLM_SCOPE(ELLMTag::Particles);

	if (Name == NAME_None)
	{
		return;
	}
	check(IsInGameThread());

	// First see if an entry for this name already exists
	for (int32 i = 0; i < InstanceParameters.Num(); i++)
	{
		FParticleSysParam& P = InstanceParameters[i];
		if (P.Name == Name && P.ParamType == PSPT_Vector)
		{
			P.Vector = Param;
			return;
		}
	}

	// We didn't find one, so create a new one.
	int32 NewParamIndex = InstanceParameters.AddZeroed();
	InstanceParameters[NewParamIndex].Name = Name;
	InstanceParameters[NewParamIndex].ParamType = PSPT_Vector;
	InstanceParameters[NewParamIndex].Vector = Param;
}


void UParticleSystemComponent::SetVectorRandParameter(FName ParameterName,const FVector& Param,const FVector& ParamLow)
{
	LLM_SCOPE(ELLMTag::Particles);

	if (ParameterName == NAME_None)
	{
		return;
	}
	check(IsInGameThread());

	// First see if an entry for this name already exists
	for (int32 i = 0; i < InstanceParameters.Num(); i++)
	{
		FParticleSysParam& P = InstanceParameters[i];
		if (P.Name == ParameterName && P.ParamType == PSPT_VectorRand)
		{
			P.Vector = Param;
			P.Vector_Low = ParamLow;
			return;
		}
	}

	// We didn't find one, so create a new one.
	int32 NewParamIndex = InstanceParameters.AddZeroed();
	InstanceParameters[NewParamIndex].Name = ParameterName;
	InstanceParameters[NewParamIndex].ParamType = PSPT_VectorRand;
	InstanceParameters[NewParamIndex].Vector = Param;
	InstanceParameters[NewParamIndex].Vector_Low = ParamLow;
}

void UParticleSystemComponent::SetVectorUnitRandParameter(FName ParameterName, const FVector& Param, const FVector& ParamLow)
{
	LLM_SCOPE(ELLMTag::Particles);

	if (ParameterName == NAME_None)
	{
		return;
	}
	check(IsInGameThread());

	// First see if an entry for this name already exists
	for (int32 i = 0; i < InstanceParameters.Num(); i++)
	{
		FParticleSysParam& P = InstanceParameters[i];
		if (P.Name == ParameterName && P.ParamType == PSPT_VectorUnitRand)
		{
			P.Vector = Param;
			P.Vector_Low = ParamLow;
			return;
		}
	}

	// We didn't find one, so create a new one.
	int32 NewParamIndex = InstanceParameters.AddZeroed();
	InstanceParameters[NewParamIndex].Name = ParameterName;
	InstanceParameters[NewParamIndex].ParamType = PSPT_VectorUnitRand;
	InstanceParameters[NewParamIndex].Vector = Param;
	InstanceParameters[NewParamIndex].Vector_Low = ParamLow;
}


void UParticleSystemComponent::SetColorParameter(FName Name, FLinearColor Param)
{
	LLM_SCOPE(ELLMTag::Particles);

	if(Name == NAME_None)
	{
		return;
	}
	check(IsInGameThread());

	FColor NewColor(Param.ToFColor(true));

	// First see if an entry for this name already exists
	for (int32 i = 0; i < InstanceParameters.Num(); i++)
	{
		FParticleSysParam& P = InstanceParameters[i];
		if (P.Name == Name && P.ParamType == PSPT_Color)
		{
			P.Color = NewColor;
			return;
		}
	}

	// We didn't find one, so create a new one.
	int32 NewParamIndex = InstanceParameters.AddZeroed();
	InstanceParameters[NewParamIndex].Name = Name;
	InstanceParameters[NewParamIndex].ParamType = PSPT_Color;
	InstanceParameters[NewParamIndex].Color = NewColor;
}


void UParticleSystemComponent::SetActorParameter(FName Name, AActor* Param)
{
	LLM_SCOPE(ELLMTag::Particles);

	if(Name == NAME_None)
	{
		return;
	}
	check(IsInGameThread());

	// First see if an entry for this name already exists
	for (int32 i = 0; i < InstanceParameters.Num(); i++)
	{
		FParticleSysParam& P = InstanceParameters[i];
		if (P.Name == Name && P.ParamType == PSPT_Actor)
		{
			P.Actor = Param;
			return;
		}
	}

	// We didn't find one, so create a new one.
	int32 NewParamIndex = InstanceParameters.AddZeroed();
	InstanceParameters[NewParamIndex].Name = Name;
	InstanceParameters[NewParamIndex].ParamType = PSPT_Actor;
	InstanceParameters[NewParamIndex].Actor = Param;
}


void UParticleSystemComponent::SetMaterialParameter(FName Name, UMaterialInterface* Param)
{
	LLM_SCOPE(ELLMTag::Particles);

	if(Name == NAME_None)
	{
		return;
	}
	check(IsInGameThread());

	// First see if an entry for this name already exists
	for (int32 i = 0; i < InstanceParameters.Num(); i++)
	{
		FParticleSysParam& P = InstanceParameters[i];
		if (P.Name == Name && P.ParamType == PSPT_Material)
		{
			bIsViewRelevanceDirty = bIsViewRelevanceDirty || (P.Material != Param);
			P.Material = Param;
			return;
		}
	}

	// We didn't find one, so create a new one.
	int32 NewParamIndex = InstanceParameters.AddZeroed();
	InstanceParameters[NewParamIndex].Name = Name;
	InstanceParameters[NewParamIndex].ParamType = PSPT_Material;
	bIsViewRelevanceDirty = bIsViewRelevanceDirty || (InstanceParameters[NewParamIndex].Material != Param);
	InstanceParameters[NewParamIndex].Material = Param;
}


bool UParticleSystemComponent::GetFloatParameter(const FName InName,float& OutFloat)
{
	// Always fail if we pass in no name.
	if(InName == NAME_None)
	{
		return false;
	}

	const TArray<struct FParticleSysParam>& UseInstanceParameters = GetAsyncInstanceParameters();
	for (int32 i = 0; i < UseInstanceParameters.Num(); i++)
	{
		const FParticleSysParam& Param = UseInstanceParameters[i];
		if (Param.Name == InName)
		{
			if (Param.ParamType == PSPT_Scalar)
			{
				OutFloat = Param.Scalar;
				return true;
			}
			else if (Param.ParamType == PSPT_ScalarRand)
			{
				OutFloat = Param.Scalar + (Param.Scalar_Low - Param.Scalar) * RandomStream.FRand();
				return true;
			}
		}
	}

	return false;
}


bool UParticleSystemComponent::GetVectorParameter(const FName InName,FVector& OutVector)
{
	// Always fail if we pass in no name.
	if(InName == NAME_None)
	{
		return false;
	}

	const TArray<struct FParticleSysParam>& UseInstanceParameters = GetAsyncInstanceParameters();
	for (int32 i = 0; i < UseInstanceParameters.Num(); i++)
	{
		const FParticleSysParam& Param = UseInstanceParameters[i];
		if (Param.Name == InName)
		{
			if (Param.ParamType == PSPT_Vector)
			{
				OutVector = Param.Vector;
				return true;
			}
			else if (Param.ParamType == PSPT_VectorRand)
			{
				FVector RandValue(RandomStream.FRand(), RandomStream.FRand(), RandomStream.FRand());
				OutVector = Param.Vector + (Param.Vector_Low - Param.Vector) * RandValue;
				return true;
			}
			else if (Param.ParamType == PSPT_VectorUnitRand)
			{
				return true;
			}
		}
	}

	return false;
}

bool UParticleSystemComponent::GetAnyVectorParameter(const FName InName,FVector& OutVector)
{
	// Always fail if we pass in no name.
	if(InName == NAME_None)
	{
		return false;
	}

	const TArray<struct FParticleSysParam>& UseInstanceParameters = GetAsyncInstanceParameters();
	for (int32 i = 0; i < UseInstanceParameters.Num(); i++)
	{
		const FParticleSysParam& Param = UseInstanceParameters[i];
		if (Param.Name == InName)
		{
			if (Param.ParamType == PSPT_Vector)
			{
				OutVector = Param.Vector;
				return true;
			}
			if (Param.ParamType == PSPT_VectorRand)
			{
				FVector RandValue(RandomStream.FRand(), RandomStream.FRand(), RandomStream.FRand());
				OutVector = Param.Vector + (Param.Vector_Low - Param.Vector) * RandValue;
				return true;
			}
			else if (Param.ParamType == PSPT_VectorUnitRand)
			{
				return true;
			}
			if (Param.ParamType == PSPT_Scalar)
			{
				float OutFloat = Param.Scalar;
				OutVector = FVector(OutFloat, OutFloat, OutFloat);
				return true;
			}
			if (Param.ParamType == PSPT_ScalarRand)
			{
				float OutFloat = Param.Scalar + (Param.Scalar_Low - Param.Scalar) * RandomStream.FRand();
				OutVector = FVector(OutFloat, OutFloat, OutFloat);
				return true;
			}
			if (Param.ParamType == PSPT_Color)
			{
				OutVector = FVector(FLinearColor(Param.Color));
				return true;
			}
		}
	}

	return false;
}


bool UParticleSystemComponent::GetColorParameter(const FName InName,FLinearColor& OutColor)
{
	// Always fail if we pass in no name.
	if(InName == NAME_None)
	{
		return false;
	}
	const TArray<struct FParticleSysParam>& UseInstanceParameters = GetAsyncInstanceParameters();

	for (int32 i = 0; i < UseInstanceParameters.Num(); i++)
	{
		const FParticleSysParam& Param = UseInstanceParameters[i];
		if (Param.Name == InName && Param.ParamType == PSPT_Color)
		{
			OutColor = FLinearColor(Param.Color);
			return true;
		}
	}

	return false;
}


bool UParticleSystemComponent::GetActorParameter(const FName InName,class AActor*& OutActor)
{
	// Always fail if we pass in no name.
	if (InName == NAME_None)
	{
		return false;
	}

	const TArray<struct FParticleSysParam>& UseInstanceParameters = GetAsyncInstanceParameters();
	for (int32 i = 0; i < UseInstanceParameters.Num(); i++)
	{
		const FParticleSysParam& Param = UseInstanceParameters[i];
		if (Param.Name == InName && Param.ParamType == PSPT_Actor)
		{
			OutActor = Param.Actor;
			return true;
		}
	}

	return false;
}


bool UParticleSystemComponent::GetMaterialParameter(const FName InName,class UMaterialInterface*& OutMaterial)
{
	// Always fail if we pass in no name.
	if (InName == NAME_None)
	{
		return false;
	}

	const TArray<struct FParticleSysParam>& UseInstanceParameters = GetAsyncInstanceParameters();
	for (int32 i = 0; i < UseInstanceParameters.Num(); i++)
	{
		const FParticleSysParam& Param = UseInstanceParameters[i];
		if (Param.Name == InName && Param.ParamType == PSPT_Material)
		{
			OutMaterial = Param.Material;
			return true;
		}
	}

	return false;
}


void UParticleSystemComponent::ClearParameter(FName ParameterName, EParticleSysParamType ParameterType)
{
	check(IsInGameThread());
	for (int32 i = 0; i < InstanceParameters.Num(); i++)
	{
		if (InstanceParameters[i].Name == ParameterName && (ParameterType == PSPT_None || InstanceParameters[i].ParamType == ParameterType))
		{
			InstanceParameters.RemoveAt(i--);
		}
	}
}


void UParticleSystemComponent::AutoPopulateInstanceProperties()
{
	check(IsInGameThread());
	if (Template)
	{
		for (int32 EmitterIndex = 0; EmitterIndex < Template->Emitters.Num(); EmitterIndex++)
		{
			UParticleEmitter* Emitter = Template->Emitters[EmitterIndex];
			Emitter->AutoPopulateInstanceProperties(this);
		}
	}
}

void UParticleSystemComponent::GetUsedMaterials( TArray<UMaterialInterface*>& OutMaterials, bool bGetDebugMaterials ) const
{
	if (Template)
	{
		for (int32 EmitterIdx = 0; EmitterIdx < Template->Emitters.Num(); ++EmitterIdx)
		{
			const UParticleEmitter* Emitter = Template->Emitters[EmitterIdx];
			if (!Emitter)
			{
				continue;
			}

			for (int32 LodIndex = 0; LodIndex < Emitter->LODLevels.Num(); ++LodIndex)
			{
				const UParticleLODLevel* LOD = Emitter->LODLevels[LodIndex];
				if (LOD)
				{
					LOD->GetUsedMaterials(OutMaterials, Template->NamedMaterialSlots, EmitterMaterials);
				}
			}
		}
	}

	OutMaterials.Append(EmitterMaterials);
}

typedef TPair<const UMaterialInterface*, float> FMaterialWithScale;

void AddMaterials(TArray<FMaterialWithScale, TInlineAllocator<12> >& OutMaterialWithScales, const TArray<UMaterialInterface*>& InMaterials, float InScale)
{
	for (const UMaterialInterface* Material : InMaterials)
	{
		if (Material)
		{
			FMaterialWithScale* Entry = OutMaterialWithScales.FindByPredicate([&](const FMaterialWithScale& Ref) { return Ref.Key == Material; });
			if (Entry)
			{
				Entry->Value = FMath::Max<int32>(Entry->Value, InScale);
			}
			else
			{
				new (OutMaterialWithScales) FMaterialWithScale(Material, InScale);
			}
		}
	}
}

void UParticleSystemComponent::GetStreamingRenderAssetInfo(FStreamingTextureLevelContext& LevelContext, TArray<FStreamingRenderAssetPrimitiveInfo>& OutStreamingRenderAssets) const
{
	TArray<FMaterialWithScale, TInlineAllocator<12> > MaterialWithScales;

	if (Template)
	{
		// Find the max sub uv scale of each texture as we can't apply them incrementally
		TArray<UMaterialInterface*> LODLevelMaterials;

		for (int32 EmitterIdx = 0; EmitterIdx < Template->Emitters.Num(); ++EmitterIdx)
		{
			const UParticleEmitter* Emitter = Template->Emitters[EmitterIdx];
			if (!Emitter)
			{
				continue;
			}

			for (int32 LodIndex = 0; LodIndex < Emitter->LODLevels.Num(); ++LodIndex)
			{
				const UParticleLODLevel* LOD = Emitter->LODLevels[LodIndex];
				if (!LOD || !LOD->RequiredModule)
				{
					continue;
				}

				LODLevelMaterials.Reset();
				LOD->GetUsedMaterials(LODLevelMaterials, Template->NamedMaterialSlots, EmitterMaterials);
				AddMaterials(MaterialWithScales, LODLevelMaterials, (float)FMath::Max<int32>(LOD->RequiredModule->SubImages_Horizontal, LOD->RequiredModule->SubImages_Vertical));

				LOD->GetStreamingMeshInfo(Bounds, OutStreamingRenderAssets);
			}
		}

		AddMaterials(MaterialWithScales, EmitterMaterials, 1.f);

		if (MaterialWithScales.Num())
		{
			static const FMeshUVChannelInfo UVChannelData(1.f);
			FPrimitiveMaterialInfo MaterialData;
			MaterialData.PackedRelativeBox = PackedRelativeBox_Identity;
			MaterialData.UVChannelData = &UVChannelData;

			for (const FMaterialWithScale& MaterialWithScale : MaterialWithScales)
			{
				MaterialData.Material = MaterialWithScale.Key;
				LevelContext.ProcessMaterial(Bounds, MaterialData, Bounds.SphereRadius * MaterialWithScale.Value, OutStreamingRenderAssets);
			}
		}
	}
}

FBodyInstance* UParticleSystemComponent::GetBodyInstance(FName BoneName /*= NAME_None*/, bool bGetWelded /*= true*/, int32 Index /*=INDEX_NONE*/) const
{
	return nullptr;
}

void UParticleSystemComponent::ReportEventSpawn(const FName InEventName, const float InEmitterTime, 
	const FVector InLocation, const FVector InVelocity, const TArray<UParticleModuleEventSendToGame*>& InEventData)
{
	FParticleEventSpawnData* SpawnData = new(SpawnEvents)FParticleEventSpawnData;
	SpawnData->Type = EPET_Spawn;
	SpawnData->EventName = InEventName;
	SpawnData->EmitterTime = InEmitterTime;
	SpawnData->Location = InLocation;
	SpawnData->Velocity = InVelocity;
	SpawnData->EventData = InEventData;
}

void UParticleSystemComponent::ReportEventDeath(const FName InEventName, const float InEmitterTime, 
	const FVector InLocation, const FVector InVelocity, const TArray<UParticleModuleEventSendToGame*>& InEventData, const float InParticleTime)
{
	FParticleEventDeathData* DeathData = new(DeathEvents)FParticleEventDeathData;
	DeathData->Type = EPET_Death;
	DeathData->EventName = InEventName;
	DeathData->EmitterTime = InEmitterTime;
	DeathData->Location = InLocation;
	DeathData->Velocity = InVelocity;
	DeathData->EventData = InEventData;
	DeathData->ParticleTime = InParticleTime;
}

void UParticleSystemComponent::ReportEventCollision(const FName InEventName, const float InEmitterTime, 
	const FVector InLocation, const FVector InDirection, const FVector InVelocity, const TArray<UParticleModuleEventSendToGame*>& InEventData,
	const float InParticleTime, const FVector InNormal, const float InTime, const int32 InItem, const FName InBoneName, UPhysicalMaterial* PhysMat)
{
	FParticleEventCollideData* CollideData = new(CollisionEvents)FParticleEventCollideData;
	CollideData->Type = EPET_Collision;
	CollideData->EventName = InEventName;
	CollideData->EmitterTime = InEmitterTime;
	CollideData->Location = InLocation;
	CollideData->Direction = InDirection;
	CollideData->Velocity = InVelocity;
	CollideData->EventData = InEventData;
	CollideData->ParticleTime = InParticleTime;
	CollideData->Normal = InNormal;
	CollideData->Time = InTime;
	CollideData->Item = InItem;
	CollideData->BoneName = InBoneName;
	CollideData->PhysMat = PhysMat;
}

void UParticleSystemComponent::ReportEventBurst(const FName InEventName, const float InEmitterTime, const int32 InParticleCount,
	const FVector InLocation, const TArray<UParticleModuleEventSendToGame*>& InEventData)
{
	FParticleEventBurstData* BurstData = new(BurstEvents)FParticleEventBurstData;
	BurstData->Type = EPET_Burst;
	BurstData->EventName = InEventName;
	BurstData->EmitterTime = InEmitterTime;
	BurstData->ParticleCount = InParticleCount;
	BurstData->Location = InLocation;
	BurstData->EventData = InEventData;
}

void UParticleSystemComponent::GenerateParticleEvent(const FName InEventName, const float InEmitterTime,
	const FVector InLocation, const FVector InDirection, const FVector InVelocity)
{
	FParticleEventKismetData* KismetData = new(KismetEvents)FParticleEventKismetData;
	KismetData->Type = EPET_Blueprint;
	KismetData->EventName = InEventName;
	KismetData->EmitterTime = InEmitterTime;
	KismetData->Location = InLocation;
	KismetData->Velocity = InVelocity;
}

void UParticleSystemComponent::KillParticlesForced()
{
	ForceAsyncWorkCompletion(STALL);
	for (int32 EmitterIndex=0;EmitterIndex<EmitterInstances.Num();EmitterIndex++)
	{
		if (EmitterInstances[EmitterIndex])
		{
			EmitterInstances[EmitterIndex]->KillParticlesForced();
		}
	}
}


void UParticleSystemComponent::ForceUpdateBounds()
{
	ForceAsyncWorkCompletion(STALL);
	FBox BoundingBox;

	BoundingBox.Init();

	const int32 EmitterCount = EmitterInstances.Num();
	for ( int32 EmitterIndex = 0; EmitterIndex < EmitterCount; ++EmitterIndex )
	{
		FParticleEmitterInstance* Instance = EmitterInstances[EmitterIndex];
		if ( Instance )
		{
			Instance->ForceUpdateBoundingBox();
			BoundingBox += Instance->GetBoundingBox();
		}
	}

	// Expand the actual bounding-box slightly so it will be valid longer in the case of expanding particle systems.
	const FVector ExpandAmount = BoundingBox.GetExtent() * 0.1f;
	BoundingBox = FBox(BoundingBox.Min - ExpandAmount, BoundingBox.Max + ExpandAmount);

	// Update our bounds.
	Bounds = BoundingBox;
}


bool UParticleSystemComponent::ShouldComputeLODFromGameThread()
{
	bool bUseGameThread = false;
	UWorld* World = GetWorld();
	if (World && World->IsGameWorld() && GbEnableGameThreadLODCalculation)
	{
		check(IsInGameThread());

		for (FConstPlayerControllerIterator Iterator = World->GetPlayerControllerIterator(); Iterator; ++Iterator)
		{
			APlayerController* PlayerController = Iterator->Get();
			if (PlayerController && PlayerController->IsLocalPlayerController())
			{
				bUseGameThread = true;
				break;
			}
		}
	}
	return bUseGameThread;
}


UParticleSystemReplay* UParticleSystemComponent::FindReplayClipForIDNumber( const int32 InClipIDNumber )
{
	// @todo: If we ever end up with more than a few clips, consider changing this to a hash
	for( int32 CurClipIndex = 0; CurClipIndex < ReplayClips.Num(); ++CurClipIndex )
	{
		UParticleSystemReplay* CurReplayClip = ReplayClips[ CurClipIndex ];
		if( CurReplayClip != NULL )
		{
			if( CurReplayClip->ClipIDNumber == InClipIDNumber )
			{
				// Found it!  We're done.
				return CurReplayClip;
			}
		}
	}

	// Not found
	return NULL;
}

UMaterialInstanceDynamic* UParticleSystemComponent::CreateNamedDynamicMaterialInstance(FName Name, class UMaterialInterface* SourceMaterial)
{
	int32 Index = GetNamedMaterialIndex(Name);
	if (INDEX_NONE == Index)
	{
		UE_LOGF(LogParticles, Warning, "CreateNamedDynamicMaterialInstance on %ls: This material wasn't found. Check the particle system's named material slots in cascade.", *GetPathName());
		return NULL;
	}

	if (SourceMaterial)
	{
		SetMaterial(Index, SourceMaterial);
	}

	UMaterialInterface* MaterialInstance = GetMaterial(Index);
	UMaterialInstanceDynamic* MID = Cast<UMaterialInstanceDynamic>(MaterialInstance);

	if (MaterialInstance && !MID)
	{
		// Create and set the dynamic material instance.
		MID = UMaterialInstanceDynamic::Create(MaterialInstance, this);
		SetMaterial(Index, MID);
	}
	else if (!MaterialInstance)
	{
		UE_LOGF(LogParticles, Warning, "CreateDynamicMaterialInstance on %ls: Material index %d is invalid.", *GetPathName(), Index);
	}

	return MID;
}


UMaterialInterface* UParticleSystemComponent::GetMaterialByName(FName MaterialSlotName) const
{
	return GetNamedMaterial(MaterialSlotName);
}


void UParticleSystemComponent::SetMaterialByName(FName MaterialSlotName, class UMaterialInterface* SourceMaterial)
{
	int32 Index = GetNamedMaterialIndex(MaterialSlotName);
	if (INDEX_NONE == Index)
	{
		UE_LOGF(LogParticles, Warning, "SetMaterialByName on %ls: %ls named material wasn't found. Check the particle system's named material slots in cascade.", *GetPathName(), *MaterialSlotName.ToString());
		return;
	}

	if (SourceMaterial)
	{
		SetMaterial(Index, SourceMaterial);
	}
}

UMaterialInterface* UParticleSystemComponent::GetNamedMaterial(FName Name) const
{
	int32 Index = GetNamedMaterialIndex(Name);
	if (INDEX_NONE != Index)
	{
		if (EmitterMaterials.IsValidIndex(Index) && nullptr != EmitterMaterials[Index])
		{
			//Material has been overridden externally
			return EmitterMaterials[Index];
		}
		else
		{
			//This slot hasn't been overridden so just used the default.
			return Template ? Template->NamedMaterialSlots[Index].Material : nullptr;
		}
	}
	//Could not find this named materials slot.
	return nullptr;
}

int32 UParticleSystemComponent::GetNamedMaterialIndex(FName Name) const
{
	if (Template != nullptr)
	{
		TArray<FNamedEmitterMaterial>& Slots = Template->NamedMaterialSlots;
		for (int32 SlotIdx = 0; SlotIdx < Slots.Num(); ++SlotIdx)
		{
			if (Name == Slots[SlotIdx].Name)
			{
				return SlotIdx;
			}
		}
	}
	return INDEX_NONE;
}

FName UParticleSystemComponent::GetNameForMaterial(UMaterialInterface* InMaterial) const
{
	if (Template != nullptr)
	{
		TArray<FNamedEmitterMaterial>& Slots = Template->NamedMaterialSlots;
		for (int32 SlotIdx = 0; SlotIdx < Slots.Num(); ++SlotIdx)
		{
			if (InMaterial == Slots[SlotIdx].Material)
			{
				return Slots[SlotIdx].Name;
			}
		}
	}
	return NAME_None;
}

/**
* Archive for counting struct memory
*/
class FArchiveCountStructMem : public FArchive
{
public:
	FArchiveCountStructMem()
		: Num(0), Max(0)
	{
		ArIsCountingMemory = 1;
	}
	void CountBytes(SIZE_T InNum, SIZE_T InMax)
	{
		Num += InNum;
		Max += InMax;
	}
	SIZE_T Num, Max;
};

uint32 UParticleSystemComponent::GetApproxMemoryUsage()const
{
	uint32 MemUsage = sizeof(UParticleSystemComponent);

	for (FParticleEmitterInstance* EmitterInst : EmitterInstances)
	{
		if (EmitterInst)
		{
			int32 Num;
			int32 Max;
			EmitterInst->GetAllocatedSize(Num, Max);
			MemUsage += Max;
		}
	}

	// This is buggy we are peeking into the scene proxy data and a command might be in flight to update the dynamic data
#if 0
	FParticleSystemSceneProxy* PSysSceneProxy = (FParticleSystemSceneProxy*)SceneProxy;
	if (PSysSceneProxy != NULL)
	{
		MemUsage += PSysSceneProxy->GetAllocatedSize();
		if (FParticleDynamicData* DynamicData = PSysSceneProxy->GetDynamicData())
		{
			MemUsage += DynamicData->GetMemoryFootprint();
		#if 0
			for (FDynamicEmitterDataBase* DynEmitterData : DynamicData->DynamicEmitterDataArray)
			{
				if (DynEmitterData)
				{
					//TODO: This is gonna be relatively small but maybe work adding
					//MemUsage += DynEmitterData->GetMemoryFootprint();

					FArchiveCountStructMem MemCounter;
					const_cast<FDynamicEmitterReplayDataBase&>(DynEmitterData->GetSource()).Serialize(MemCounter);

					MemUsage += MemCounter.Max;
				}
			}
		#endif
		}
	}
#endif

	return MemUsage;
}

UParticleSystemReplay::UParticleSystemReplay(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

void UParticleSystemReplay::Serialize( FArchive& Ar )
{
	LLM_SCOPE(ELLMTag::Particles);

	Super::Serialize( Ar );

	// Serialize clip ID number
	Ar << ClipIDNumber;

	// Serialize our native members
	Ar << Frames;
}

/** FParticleSystemReplayFrame serialization operator */
FArchive& operator<<( FArchive& Ar, FParticleSystemReplayFrame& Obj )
{
	if( Ar.IsLoading() )
	{
		// Zero out the struct if we're loading from disk since we won't be cleared by default
		FMemory::Memzero( &Obj, sizeof( FParticleEmitterReplayFrame ) );
	}

	// Serialize emitter frames
	Ar << Obj.Emitters;

	return Ar;
}



/** FParticleEmitterReplayFrame serialization operator */
FArchive& operator<<( FArchive& Ar, FParticleEmitterReplayFrame& Obj )
{
	if( Ar.IsLoading() )
	{
		// Zero out the struct if we're loading from disk since we won't be cleared by default
		FMemory::Memzero( &Obj, sizeof( FParticleEmitterReplayFrame ) );
	}

	// Emitter type
	Ar << Obj.EmitterType;

	// Original emitter index
	Ar << Obj.OriginalEmitterIndex;

	if( Ar.IsLoading() )
	{
		switch( Obj.EmitterType )
		{
			case DET_Sprite:
				Obj.FrameState = new FDynamicSpriteEmitterReplayData();
				break;

			case DET_Mesh:
				Obj.FrameState = new FDynamicMeshEmitterReplayData();
				break;

			case DET_Beam2:
				Obj.FrameState = new FDynamicBeam2EmitterReplayData();
				break;

			case DET_Ribbon:
				Obj.FrameState = new FDynamicRibbonEmitterReplayData();
				break;

			case DET_AnimTrail:
				Obj.FrameState = new FDynamicTrailsEmitterReplayData();
				break;

			default:
				{
					// @todo: Support other particle types
					Obj.FrameState = NULL;
				}
				break;
		}
	}

	if( Obj.FrameState != NULL )
	{
		// Serialize this emitter frame state
		Obj.FrameState->Serialize( Ar );
	}

	return Ar;
}

AEmitterCameraLensEffectBase::AEmitterCameraLensEffectBase(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer
		.DoNotCreateDefaultSubobject(TEXT("Sprite"))
		.DoNotCreateDefaultSubobject(TEXT("ArrowComponent0"))
	)
{
	InitialLifeSpan = 10.0f;
	BaseFOV = 80.0f;
	bDestroyOnSystemFinish = true;

	// default transform is a 180 yaw to flip the system around to face the camera
	// and 90 units pushed out
	// (we assume it by default that the effect was authored facing down the +X, due to legacy reasons)
	RelativeTransform = FTransform( 
		FRotator(0.f, 180.f, 0.f),
		FVector(90.f, 0.f, 0.f)
		);

	GetParticleSystemComponent()->bOnlyOwnerSee = true;
	GetParticleSystemComponent()->SecondsBeforeInactive = 0.0f;

	// this property is deprecated, give it the sentinel value to indicate it doesn't need to be migrated
	DistFromCamera_DEPRECATED = TNumericLimits<float>::Max();
	bResetWhenRetriggered = false;
}


FTransform AEmitterCameraLensEffectBase::GetAttachedEmitterTransform(AEmitterCameraLensEffectBase const* Emitter, const FVector& CamLoc, const FRotator& CamRot, float CamFOVDeg)
{
	return ICameraLensEffectInterface::GetAttachedEmitterTransform(Emitter, CamLoc, CamRot, CamFOVDeg);
}

void AEmitterCameraLensEffectBase::UpdateLocation(const FVector& CamLoc, const FRotator& CamRot, float CamFOVDeg)
{
	FTransform const EffectToWorld = ICameraLensEffectInterface::GetAttachedEmitterTransform(this, CamLoc, CamRot, CamFOVDeg);
	SetActorTransform(EffectToWorld);
}

void AEmitterCameraLensEffectBase::EndPlay(const EEndPlayReason::Type EndPlayReason)
{
	if (BaseCamera != NULL)
	{
		BaseCamera->RemoveGenericCameraLensEffect(this);
	}
	Super::EndPlay(EndPlayReason);
}

void AEmitterCameraLensEffectBase::RegisterCamera(APlayerCameraManager* C)
{
	BaseCamera = C;
}

void AEmitterCameraLensEffectBase::NotifyRetriggered() 
{
	UParticleSystemComponent* const PSC = GetParticleSystemComponent();
	if (PSC && (PSC->bWasDeactivated || bResetWhenRetriggered))
	{
		PSC->Activate(bResetWhenRetriggered);
	}
}

void AEmitterCameraLensEffectBase::PostInitializeComponents()
{
	LLM_SCOPE(ELLMTag::Particles);

	GetParticleSystemComponent()->SetDepthPriorityGroup(SDPG_Foreground);
	Super::PostInitializeComponents();
	ActivateLensEffect();
}

void AEmitterCameraLensEffectBase::PostLoad()
{
	LLM_SCOPE(ELLMTag::Particles);

	Super::PostLoad();

	// using TNumericLimits<float>::Max() as a sentinel value to indicate this deprecated data has been 
	// migrated to the new format
	if (DistFromCamera_DEPRECATED != TNumericLimits<float>::Max())
	{
		// copy old data into the new transform
		FVector Loc = RelativeTransform.GetLocation();
		Loc.X = DistFromCamera_DEPRECATED;
		RelativeTransform.SetLocation(Loc);

		// don't copy again (just in case this gets saved, which is shouldn't)
		DistFromCamera_DEPRECATED = TNumericLimits<float>::Max();
	}
}

void AEmitterCameraLensEffectBase::ActivateLensEffect()
{
	// only play the camera effect on clients
	UWorld const* const World = GetWorld();
	check(World);
	if( !IsNetMode(NM_DedicatedServer) )
	{
		if (PS_CameraEffect)
		{
			SetTemplate( PS_CameraEffect );
		}
	}
}

void AEmitterCameraLensEffectBase::DeactivateLensEffect()
{
	UParticleSystemComponent* const PSC = GetParticleSystemComponent();
	if (PSC)
	{
		PSC->DeactivateSystem();
	}
}

bool AEmitterCameraLensEffectBase::IsLooping() const
{
	if ((PS_CameraEffect != nullptr) && PS_CameraEffect->IsLooping())
	{
		return true;
	}

	return false;
}



const FTransform& AEmitterCameraLensEffectBase::GetRelativeTransform() const
{
	return RelativeTransform;
}


float AEmitterCameraLensEffectBase::GetBaseFOV() const
{
	return BaseFOV;
}


bool AEmitterCameraLensEffectBase::ShouldAllowMultipleInstances() const
{
	return bAllowMultipleInstances;
}


bool AEmitterCameraLensEffectBase::ResetWhenTriggered() const
{
	return bResetWhenRetriggered;
}


bool AEmitterCameraLensEffectBase::ShouldTreatEmitterAsSame(TSubclassOf<AActor> OtherEmitter) const
{
	return OtherEmitter && (OtherEmitter == GetClass() || EmittersToTreatAsSame.Find(OtherEmitter) != INDEX_NONE);
}

void AEmitterCameraLensEffectBase::NotifyWillBePooled()
{
	bDestroyOnSystemFinish = false;
}

void AEmitterCameraLensEffectBase::AdjustBaseFOV(float NewFOV)
{
	BaseFOV = NewFOV;
}

//////////////////////////////////////////////////////////////////////////

void FParticleResetContext::AddTemplate(UParticleSystem* Template)
{
	check(Template);
	SystemsToReset.AddUnique(Template);
}

void FParticleResetContext::AddTemplate(UParticleModule* Module)
{
	check(Module);
	UParticleSystem* Template = Module->GetTypedOuter<UParticleSystem>();
	check(Template);
	SystemsToReset.Add(Template);
}

void FParticleResetContext::AddTemplate(UParticleEmitter* Emitter)
{
	check(Emitter);
	UParticleSystem* Template = Emitter->GetTypedOuter<UParticleSystem>();
	check(Template);
	SystemsToReset.Add(Template);
}

FParticleResetContext::~FParticleResetContext()
{
	for (TObjectIterator<UParticleSystemComponent> PSCIt; PSCIt; ++PSCIt)
	{
		UParticleSystemComponent* PSC = *PSCIt;
		check(PSC);

		if (SystemsToReset.Contains(PSC->Template))
		{
			PSC->ResetNextTick();
		}
	}
}
//////////////////////////////////////////////////////////////////////////

FAutoConsoleCommand GDumpPSCStateCommand(
	TEXT("fx.DumpPSCTickStateInfo"),
	TEXT("Dumps state information for all current Particle System Components."),
	FConsoleCommandDelegate::CreateStatic(
		[]()
{
	struct FPSCInfo
	{
		UParticleSystemComponent* PSC;		
		bool bIsActive;
		bool bIsSignificant;
		bool bIsVisible;
		int32 NumActiveParticles;
		FPSCInfo()
			: PSC((UParticleSystemComponent*)0xDEADBEEFDEADBEEF)
			, bIsActive(false)
			, bIsSignificant(false)
			, bIsVisible(false)
			, NumActiveParticles(0)
		{}
	};

	struct FPSCInfoSummary
	{
		TArray<FPSCInfo> Components;
		int32 NumTicking;
		int32 NumManaged;
		int32 NumTickingNoTemplate;
		int32 NumTickingButInactive;
		int32 NumTickingButInvisible;
		int32 NumTickingButNonSignificant;
		int32 NumTickingNoEmitters;
		int32 NumPooled;

		FPSCInfoSummary()
			: NumTicking(0)
			, NumManaged(0)
			, NumTickingNoTemplate(0)
			, NumTickingButInactive(0)
			, NumTickingButInvisible(0)
			, NumTickingButNonSignificant(0)
			, NumTickingNoEmitters(0)
			, NumPooled(0)
		{}
	};

	struct FPSCWorldInfo
	{
		TMap<UParticleSystem*, FPSCInfoSummary> SummaryMap;
		int32 TotalPSCs;
		int32 TotalTicking;
		int32 TotalManaged;
		int32 TotalTickingNoTemplate;
		int32 TotalTickingButInactive;
		int32 TotalTickingButInvisible;
		int32 TotalTickingButNonSignificant;
		int32 TotalTickingNoEmitters;
		int32 TotalPooled;

		FPSCWorldInfo()
			: TotalPSCs(0)
			, TotalTicking(0)
			, TotalManaged(0)
			, TotalTickingNoTemplate(0)
			, TotalTickingButInactive(0)
			, TotalTickingButInvisible(0)
			, TotalTickingButNonSignificant(0)
			, TotalTickingNoEmitters(0)
			, TotalPooled(0)
		{}
	};

	//First attempt to pull out ticking emitters that aren't doing anything useful.
	TMap<UWorld*, FPSCWorldInfo> InfoMap;

	for (TObjectIterator<UParticleSystemComponent> PSCIt; PSCIt; ++PSCIt)
	{
		UParticleSystemComponent* PSC = *PSCIt;
		check(PSC);

		UWorld* World = PSC->GetWorld();
		UParticleSystem* Sys = PSC->Template;
		FPSCWorldInfo& WorldInfo = InfoMap.FindOrAdd(World);
		FPSCInfoSummary& Info = WorldInfo.SummaryMap.FindOrAdd(Sys);

		int32 PSCInfoIndex = Info.Components.AddDefaulted();
		FPSCInfo& PSCInfo = Info.Components[PSCInfoIndex];
		PSCInfo.PSC = PSC;

		++WorldInfo.TotalPSCs;

		if (PSC->IsComponentTickEnabled())
		{
			int32 NumParticles = PSC->GetNumActiveParticles();

			PSCInfo.NumActiveParticles = NumParticles;

			if (PSC->IsTickManaged())
			{
				++Info.NumManaged;
				++WorldInfo.TotalManaged;
			}
			else
			{
				++Info.NumTicking;
				++WorldInfo.TotalTicking;
			}

			if (PSC->Template == nullptr)
			{
				++Info.NumTickingNoTemplate;
				++WorldInfo.TotalTickingNoTemplate;
			}

			if (PSC->EmitterInstances.Num() == 0)
			{
				++Info.NumTickingNoEmitters;
				++WorldInfo.TotalTickingNoEmitters;
			}

			if (PSC->IsActive())
			{
				PSCInfo.bIsActive = true;
			}
			else
			{
				++Info.NumTickingButInactive;
				++WorldInfo.TotalTickingButInactive;
				PSCInfo.bIsActive = false;
			}

			PSCInfo.bIsVisible = !PSC->CanConsiderInvisible();
			if (!PSCInfo.bIsVisible)
			{
				++Info.NumTickingButInvisible;
				++WorldInfo.TotalTickingButInvisible;
			}

			if (PSC->bIsManagingSignificance)
			{
				uint32 NumSignificantEmitters = 0;
				for (UParticleEmitter* Emitter : PSC->Template->Emitters)
				{
					if (Emitter->IsSignificant(PSC->RequiredSignificance))
					{
						++NumSignificantEmitters;
					}
				}

				PSCInfo.bIsSignificant = NumSignificantEmitters > 0;
				if (NumSignificantEmitters == 0 && NumParticles == 0)
				{
					++Info.NumTickingButNonSignificant;
					++WorldInfo.TotalTickingButNonSignificant;
				}
			}
			else
			{
				PSCInfo.bIsSignificant = true;
				//I don't view this as a worry so not including in this data.s
// 				if (PSC->IsActive() || (PSC->bWasActive && !PSC->bWasCompleted))
// 				{
// 					//Check if we actually should be managing significance.
// 					if (Sys->ShouldManageSignificance())
// 					{
// 						++Info.NumBadManagingSignificance;
// 					}
// 				}
			}
		}
	}

	auto PrintPSCInfo = [](const UParticleSystem* Sys, FPSCInfoSummary& Info)
	{
		float KBUsed = (sizeof(UParticleSystemComponent) * Info.Components.Num()) / 1024.0f;
		FString MaxSigName;
		if (Sys)
		{
			switch (Sys->GetHighestSignificance())
			{
			case EParticleSignificanceLevel::Critical: MaxSigName = TEXT("Crit"); break;
			case EParticleSignificanceLevel::High: MaxSigName = TEXT("High"); break;
			case EParticleSignificanceLevel::Medium: MaxSigName = TEXT("Med"); break;
			case EParticleSignificanceLevel::Low: MaxSigName = TEXT("Low"); break;
			}
		}

		UE_LOGF(LogParticles, Log, "| %5u | %7.2f | %7u | %7u | %8u | %9u || %4d | %6ls |%ls",
			Info.Components.Num(),
			KBUsed,
			Info.NumTicking,
			Info.NumManaged,
			Info.NumTickingButInactive,
			Info.NumTickingButInvisible,
			Sys ? Sys->IsLooping() : 0,
			*MaxSigName,
			Sys ? *Sys->GetFullName() : TEXT("NULL SYSTEM!"));
	};

	for (TPair<UWorld*, FPSCWorldInfo> InfoMapPair : InfoMap)
	{
		UWorld* World = InfoMapPair.Key;
		FPSCWorldInfo& WorldInfo = InfoMapPair.Value;

		FString WorldInfoString;

		if (World)
		{
#define WORLD_TYPE_CASE(WorldType) case EWorldType::WorldType: WorldInfoString += TEXT(#WorldType); break;
			switch (World->WorldType)
			{
				WORLD_TYPE_CASE(None)
				WORLD_TYPE_CASE(Game)
				WORLD_TYPE_CASE(Editor)
				WORLD_TYPE_CASE(PIE)
				WORLD_TYPE_CASE(EditorPreview)
				WORLD_TYPE_CASE(GamePreview)
				WORLD_TYPE_CASE(GameRPC)
				WORLD_TYPE_CASE(Inactive)
			};
#undef WORLD_TYPE_CASE

			WorldInfoString += TEXT(" | ");
			WorldInfoString += World->GetFullName();
		}

		float KBUsed = (sizeof(UParticleSystemComponent) * WorldInfo.TotalPSCs) / 1024.0f;

		UE_LOGF(LogParticles, Log, "|-------------------------------------------------------------------------------------------------------|");
		UE_LOGF(LogParticles, Log, "|	   	                  Particle System Component Tick State Info                                     |");
		UE_LOGF(LogParticles, Log, "|-------------------------------------------------------------------------------------------------------|");
		UE_LOGF(LogParticles, Log, "| World: 0x%p - %ls |", World, *WorldInfoString);
		UE_LOGF(LogParticles, Log, "|-------------------------------------------------------------------------------------------------------|");
		UE_LOGF(LogParticles, Log, "| Inactive = Ticking but is not active and has no active particles.  This should be investigated.                                   |");
		UE_LOGF(LogParticles, Log, "| Invisible = Ticking but is not visible. Ideally these systems could be culled by the significance manager but this requires them to be non critical.   |");
		UE_LOGF(LogParticles, Log, "|-------------------------------------------------------------------------------------------------------|");
		UE_LOGF(LogParticles, Log, "|                                            Summary                                                    |");
		UE_LOGF(LogParticles, Log, "|-------------------------------------------------------------------------------------------------------|");
		UE_LOGF(LogParticles, Log, "| Total | Mem(KB) | Ticking | Managed | Inactive | Invisible | Template |---------||");
		UE_LOGF(LogParticles, Log, "| %5u | %7.2f | %7u | %7u | %8u | %9u|| Loop | MaxSig | Name |",
			WorldInfo.TotalPSCs, KBUsed, WorldInfo.TotalTicking, WorldInfo.TotalManaged, WorldInfo.TotalTickingButInactive, WorldInfo.TotalTickingButInvisible);
		UE_LOGF(LogParticles, Log, "|-------------------------------------------------------------------------------------------------------|");

// 		FPSCInfoSummary* NullInfo = WorldInfo.SummaryMap.Find(nullptr);
// 		if (NullInfo)
// 		{
// 			PrintPSCInfo(nullptr, *NullInfo);
// 		}

		WorldInfo.SummaryMap.ValueSort([](const FPSCInfoSummary& A, const FPSCInfoSummary& B) { return ((A.Components.Num()/1000.0f) + (A.NumManaged + A.NumTicking)) > ((B.Components.Num() / 1000.0f) + (B.NumManaged + B.NumTicking)); });

		for (TPair <UParticleSystem*, FPSCInfoSummary >& Pair : WorldInfo.SummaryMap)
		{
			const UParticleSystem* Sys = Pair.Key;
			FPSCInfoSummary& Info = Pair.Value;
		//	if (Sys)
			{
				PrintPSCInfo(Sys, Info);
			}
		}

		//Now dump the full list of ticking components by system.
		UE_LOGF(LogParticles, Log, "|-------------------------------------------------------------------------------------------|");
		UE_LOGF(LogParticles, Log, "|-- All Ticking or Managed Components By System --------------------------------------------|");
		UE_LOGF(LogParticles, Log, "|-------------------------------------------------------------------------------------------|");
		for (TPair <UParticleSystem*, FPSCInfoSummary >& Pair : WorldInfo.SummaryMap)
		{
			const UParticleSystem* Sys = Pair.Key;
			FPSCInfoSummary& Info = Pair.Value;

			if (Info.NumManaged > 0 || Info.NumTicking > 0)
			{
				UE_LOGF(LogParticles, Log, "|-- Sys: %ls -------------------------------------------------------|", Sys ? *Sys->GetFullName() : TEXT("null"));

				//Sort to bring ticking but inactive components to the top.
				Info.Components.Sort([](const FPSCInfo& A, const FPSCInfo& B) { return !A.bIsActive + !A.bIsSignificant + !A.bIsVisible < !B.bIsActive + !B.bIsSignificant + !B.bIsVisible; });
				for (FPSCInfo& PSCInfo : Info.Components)
				{
					bool bTickManaged = PSCInfo.PSC->IsTickManaged();
					if (PSCInfo.PSC->IsComponentTickEnabled())
					{
						UE_LOGF(LogParticles, Log, "| PSC: %p | Ticking: %d | Managed: %d | Active: %d | Sig: %d | Vis: %d | Num: %d | %ls", PSCInfo.PSC, !bTickManaged, bTickManaged, PSCInfo.bIsActive, PSCInfo.bIsSignificant, PSCInfo.bIsVisible, PSCInfo.NumActiveParticles, *PSCInfo.PSC->GetFullName());
					}
				}
			}
		}
	}
})
);

FPSCTickData& UParticleSystemComponent::GetManagerTickData()
{
	return GetWorldManager()->GetTickData(ManagerHandle);
};

FParticleSystemWorldManager* UParticleSystemComponent::GetWorldManager()const
{
	return FParticleSystemWorldManager::Get(GetWorld());
}

void UParticleSystemComponent::SetAutoAttachmentParameters(USceneComponent* Parent, FName SocketName, EAttachmentRule LocationRule, EAttachmentRule RotationRule, EAttachmentRule ScaleRule)
{
	AutoAttachParent = Parent;
	AutoAttachSocketName = SocketName;
	AutoAttachLocationRule = LocationRule;
	AutoAttachRotationRule = RotationRule;
	AutoAttachScaleRule = ScaleRule;
}

#undef LOCTEXT_NAMESPACE

