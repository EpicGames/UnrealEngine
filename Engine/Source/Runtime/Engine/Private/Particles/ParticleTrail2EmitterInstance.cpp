// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	ParticleTrail2EmitterInstance.cpp: 
	Particle trail2 emitter instance implementation.
=============================================================================*/

#include "Components/MeshComponent.h"
#include "Engine/World.h"
#include "Materials/Material.h"
#include "MaterialDomain.h"
#include "ParticleEmitterInstances.h"
#include "ParticleEmitterInstanceOwner.h"
#include "Particles/ParticleEmitter.h"
#include "Particles/ParticleSystemComponent.h"
#include "Particles/Event/ParticleModuleEventGenerator.h"
#include "Particles/Lifetime/ParticleModuleLifetime.h"
#include "Particles/ParticleModule.h"
#include "Particles/Spawn/ParticleModuleSpawn.h"
#include "Particles/ParticleSystem.h"
#include "Particles/Spawn/ParticleModuleSpawnPerUnit.h"
#include "Particles/Spawn/ParticleModuleSpawnBase.h"
#include "Particles/Trail/ParticleModuleTrailSource.h"
#include "Particles/TypeData/ParticleModuleTypeDataBase.h"
#include "Particles/TypeData/ParticleModuleTypeDataAnimTrail.h"
#include "Particles/TypeData/ParticleModuleTypeDataRibbon.h"
#include "Particles/ParticleLODLevel.h"
#include "Particles/ParticleModuleRequired.h"
#include "Scalability.h"
#include "Stats/StatsTrace.h"
/** trail stats */


DEFINE_STAT(STAT_TrailParticles);
DEFINE_STAT(STAT_TrailParticlesRenderCalls);
DEFINE_STAT(STAT_TrailParticlesSpawned);
DEFINE_STAT(STAT_TrailParticlesTickCalls);
DEFINE_STAT(STAT_TrailParticlesKilled);
DEFINE_STAT(STAT_TrailParticlesTrianglesRendered);

DEFINE_STAT(STAT_TrailFillVertexTime);
DEFINE_STAT(STAT_TrailFillIndexTime);
DEFINE_STAT(STAT_TrailRenderingTime);
DEFINE_STAT(STAT_TrailTickTime);

DEFINE_STAT(STAT_AnimTrailNotifyTime);

DECLARE_CYCLE_STAT(TEXT("TrailEmitterInstance Init GT"), STAT_TrailEmitterInstance_Init, STATGROUP_Particles);

static int32 GSkipRibbonSpawnInterp = 1;
static FAutoConsoleVariableRef CVarSkipRibbonSpawnInterp(
	TEXT("r.Emitter.SkipRibbonSpawnInterp"),
	GSkipRibbonSpawnInterp,
	TEXT("Ignore velocity based offsets when interpolating. This prevents ribbon quads from overlapping eachother (default=1)"),
	ECVF_Default
);

#define MAX_TRAIL_INDICES	65535

namespace FXConsoleVariables
{
	int32 MaxDistanceTessellation = MAX_TRAIL_INDICES;
	int32 MaxTangentTessellation = MAX_TRAIL_INDICES;

	FAutoConsoleVariableRef CVarMaxDistanceTessellation(
		TEXT("FX.Trail.MaxDistanceTessellation"),
		MaxDistanceTessellation,
		TEXT("Maximum tessellation steps allowed for distance based tessellation."),
		ECVF_Default
	);

	FAutoConsoleVariableRef CVarMaxTangentTessellation(
		TEXT("FX.Trail.MaxTangentTessellation"),
		MaxTangentTessellation,
		TEXT("Maximum tessellation steps allowed for tangent based tessellation."),
		ECVF_Default
	);
}

/*-----------------------------------------------------------------------------
	FParticleTrailsEmitterInstance_Base.
-----------------------------------------------------------------------------*/
void FParticleTrailsEmitterInstance_Base::Init()
{
	SCOPE_CYCLE_COUNTER(STAT_TrailEmitterInstance_Init);
	FParticleEmitterInstance::Init();
	SetupTrailModules();
}

void FParticleTrailsEmitterInstance_Base::InitParameters(UParticleEmitter* InTemplate)
{
	FParticleEmitterInstance::InitParameters(InTemplate);
	if (GIsEditor)
	{
		UParticleLODLevel* LODLevel	= InTemplate->GetLODLevel(0);
		check(LODLevel);
		CurrentMaterial = LODLevel->RequiredModule->Material;
	}
}

void FParticleTrailsEmitterInstance_Base::Tick(float DeltaTime, bool bSuppressSpawning)
{
	SCOPE_CYCLE_COUNTER(STAT_TrailTickTime);
	{

#if defined(ULTRA_VERBOSE_TRAILS_DEBUG) && ULTRA_VERBOSE_TRAILS_DEBUG
		PrintTrails();
#endif
		check(SpriteTemplate);
		check(SpriteTemplate->LODLevels.Num() > 0);

		// If this the FirstTime we are being ticked?
		bool bFirstTime = (SecondsSinceCreation > 0.0f) ? false : true;

		// Grab the current LOD level
		UParticleLODLevel* LODLevel = SpriteTemplate->GetCurrentLODLevel(this);
		check(LODLevel);

// 		if (ActiveParticles == 0)
// 		{
// 			RunningTime = 0.0f;
// 		}
		check(DeltaTime >= 0.0f);

		// Handle EmitterTime setup, looping, etc.
		float EmitterDelay = Tick_EmitterTimeSetup(DeltaTime, LODLevel);

		// Update the source data (position, etc.)
		UpdateSourceData(DeltaTime, bFirstTime);

		// Kill off any dead particles
		KillParticles();

		// Spawn Particles...
		SpawnFraction = Tick_SpawnParticles(DeltaTime, LODLevel, bSuppressSpawning, bFirstTime);

		// Reset particle parameters.
		ResetParticleParameters(DeltaTime);

		// Update existing particles (might respawn dying ones).
		Tick_ModuleUpdate(DeltaTime, LODLevel);

		// Module post update 
		Tick_ModulePostUpdate(DeltaTime, LODLevel);

		// Update the orbit data...
		// 	UpdateOrbitData(DeltaTime);

		// Calculate bounding box and simulate velocity.
		UpdateBoundingBox(DeltaTime);

		// Perform any final updates...
		Tick_ModuleFinalUpdate(DeltaTime, LODLevel);

		// Recalculate tangents, if enabled
		Tick_RecalculateTangents(DeltaTime, LODLevel);

		CurrentMaterial = LODLevel->RequiredModule->Material;

		// Invalidate the contents of the vertex/index buffer.
		IsRenderDataDirty = 1;

		// 'Reset' the emitter time so that the delay functions correctly
		EmitterTime += CurrentDelay;
		RunningTime += DeltaTime;
		LastTickTime = Component.GetWorldTimeSeconds();
		
		// Reset particles position offset
		PositionOffsetThisTick = FVector::ZeroVector;
	}
	INC_DWORD_STAT(STAT_TrailParticlesTickCalls);
}

/**
 *	Tick sub-function that handles recalculation of tangents
 *
 *	@param	DeltaTime			The current time slice
 *	@param	CurrentLODLevel		The current LOD level for the instance
 */
void FParticleTrailsEmitterInstance_Base::Tick_RecalculateTangents(float DeltaTime, UParticleLODLevel* InCurrentLODLevel)
{
}

void FParticleTrailsEmitterInstance_Base::UpdateBoundingBox(float DeltaTime)
{
	SCOPE_CYCLE_COUNTER(STAT_ParticleUpdateBounds);
	{
		const UParticleSystem* Template = Component.GetTemplate();
		bool bUpdateBox = ((Component.GetIsWarmingUp() == false) && (Template != NULL) && (Template->bUseFixedRelativeBoundingBox == false));
		// Handle local space usage
		check(SpriteTemplate->LODLevels.Num() > 0);
		UParticleLODLevel* LODLevel = SpriteTemplate->LODLevels[0];
		check(LODLevel);

		if (bUpdateBox)
		{
			// Set the min/max to the position of the trail
			if (LODLevel->RequiredModule->bUseLocalSpace == false) 
			{
				ParticleBoundingBox.Max = Component.GetComponentTransform().GetLocation();
				ParticleBoundingBox.Min = Component.GetComponentTransform().GetLocation();
			}
			else
			{
				ParticleBoundingBox.Max = FVector::ZeroVector;
				ParticleBoundingBox.Min = FVector::ZeroVector;
			}
		}
		ParticleBoundingBox.IsValid = true;

		// Take scale into account
		FVector Scale = Component.GetComponentTransform().GetScale3D();

		// As well as each particle
		int32 LocalActiveParticles = ActiveParticles;
		if (LocalActiveParticles > 0)
		{
			FVector MinPos(FLT_MAX);
			FVector MaxPos(-FLT_MAX);
			FVector TempMin;
			FVector TempMax;
			bool bSkipDoubleSpawnUpdate = !SpriteTemplate->bUseLegacySpawningBehavior;
			for (int32 i = 0; i < LocalActiveParticles; i++)
			{
				DECLARE_PARTICLE_PTR(Particle, ParticleData + ParticleStride * ParticleIndices[i]);
				FVector Size = (FVector)Particle->Size * Scale;
				// Do linear integrator and update bounding box
				bool bJustSpawned = (Particle->Flags & STATE_Particle_JustSpawned) != 0;
				Particle->Flags &= ~STATE_Particle_JustSpawned;

				//Don't update position for newly spawned particles. They already have a partial update applied during spawn.
				bool bSkipUpdate = bJustSpawned && bSkipDoubleSpawnUpdate;
				Particle->Location	+= bSkipUpdate ? FVector::ZeroVector : FVector(DeltaTime * Particle->Velocity);
				Particle->Rotation	+= bSkipUpdate ? 0.0f : DeltaTime * Particle->RotationRate;
				Particle->Location	+= PositionOffsetThisTick;
				FPlatformMisc::Prefetch(ParticleData, (ParticleIndices[i+1] * ParticleStride));
				FPlatformMisc::Prefetch(ParticleData, (ParticleIndices[i+1] * ParticleStride) + PLATFORM_CACHE_LINE_SIZE);
				Particle->OldLocation = Particle->Location;
				if (bUpdateBox)
				{
					TempMin = Particle->Location - Size;
					TempMax = Particle->Location + Size;
					MinPos.X = FMath::Min(TempMin.X, MinPos.X);
					MinPos.Y = FMath::Min(TempMin.Y, MinPos.Y);
					MinPos.Z = FMath::Min(TempMin.Z, MinPos.Z);
					MaxPos.X = FMath::Max(TempMin.X, MaxPos.X);
					MaxPos.Y = FMath::Max(TempMin.Y, MaxPos.Y);
					MaxPos.Z = FMath::Max(TempMin.Z, MaxPos.Z);
					MinPos.X = FMath::Min(TempMax.X, MinPos.X);
					MinPos.Y = FMath::Min(TempMax.Y, MinPos.Y);
					MinPos.Z = FMath::Min(TempMax.Z, MinPos.Z);
					MaxPos.X = FMath::Max(TempMax.X, MaxPos.X);
					MaxPos.Y = FMath::Max(TempMax.Y, MaxPos.Y);
					MaxPos.Z = FMath::Max(TempMax.Z, MaxPos.Z);
				}

				// Do angular integrator, and wrap result to within +/- 2 PI
				Particle->Rotation	 = FMath::Fmod(Particle->Rotation, 2.f*(float)UE_PI);
			}
			if (bUpdateBox)
			{
				ParticleBoundingBox += MinPos;
				ParticleBoundingBox += MaxPos;
			}
		}

		// Transform bounding box into world space if the emitter uses a local space coordinate system.
		if (bUpdateBox)
		{
			if (LODLevel->RequiredModule->bUseLocalSpace) 
			{
				ParticleBoundingBox = ParticleBoundingBox.TransformBy(Component.GetComponentTransform());
			}
		}
	}
}

/**
 * Force the bounding box to be updated.
 */
void FParticleTrailsEmitterInstance_Base::ForceUpdateBoundingBox()
{
	{
		// Handle local space usage
		check(SpriteTemplate->LODLevels.Num() > 0);
		UParticleLODLevel* LODLevel = SpriteTemplate->LODLevels[0];
		check(LODLevel);

		// Set the min/max to the position of the trail
		if (LODLevel->RequiredModule->bUseLocalSpace == false) 
		{
			ParticleBoundingBox.Max = Component.GetComponentTransform().GetLocation();
			ParticleBoundingBox.Min = Component.GetComponentTransform().GetLocation();
		}
		else
		{
			ParticleBoundingBox.Max = FVector::ZeroVector;
			ParticleBoundingBox.Min = FVector::ZeroVector;
		}
		ParticleBoundingBox.IsValid = true;

		// Take scale into account
		FVector Scale = Component.GetComponentTransform().GetScale3D();

		// As well as each particle
		int32 LocalActiveParticles = ActiveParticles;
		if (LocalActiveParticles > 0)
		{
			FVector MinPos(FLT_MAX);
			FVector MaxPos(-FLT_MAX);
			FVector TempMin;
			FVector TempMax;
			for (int32 i = 0; i < LocalActiveParticles; i++)
			{
				DECLARE_PARTICLE_PTR(Particle, ParticleData + ParticleStride * ParticleIndices[i]);
				FVector AbsSize = ((FVector)Particle->Size * Scale).GetAbs();
				TempMin = Particle->Location - AbsSize;
				TempMax = Particle->Location + AbsSize;
				MinPos = TempMin.ComponentMin(MinPos);
				MaxPos = TempMin.ComponentMax(MaxPos);
			}

			ParticleBoundingBox += MinPos;
			ParticleBoundingBox += MaxPos;
		}

		// Transform bounding box into world space if the emitter uses a local space coordinate system.
		if (LODLevel->RequiredModule->bUseLocalSpace) 
		{
			ParticleBoundingBox = ParticleBoundingBox.TransformBy(Component.GetComponentTransform());
		}
	}
}

void FParticleTrailsEmitterInstance_Base::UpdateSourceData(float DeltaTime, bool bFirstTime)
{
}

void FParticleTrailsEmitterInstance_Base::KillParticles()
{
	if (ActiveParticles > 0)
	{
		float CurrentTickTime = Component.GetWorldTimeSeconds();
		bool bHasForceKillParticles = false;
		// Loop over the active particles... If their RelativeTime is > 1.0f (indicating they are dead),
		// move them to the 'end' of the active particle list.
		for (int32 ParticleIdx = ActiveParticles - 1; ParticleIdx >= 0; ParticleIdx--)
		{
			const int32 CurrentIndex = ParticleIndices[ParticleIdx];

			DECLARE_PARTICLE_PTR(Particle, ParticleData + ParticleStride * CurrentIndex);
			FTrailsBaseTypeDataPayload* TrailData = ((FTrailsBaseTypeDataPayload*)((uint8*)Particle + TypeDataOffset));
			if ((Particle->RelativeTime > 1.0f) ||
				((bEnableInactiveTimeTracking == true) && 
				 (CurrentTickTime != 0.0f) && 
				 ((CurrentTickTime - LastTickTime) > (1.0f / Particle->OneOverMaxLifetime)))
				)
			{
#if defined(_TRAILS_DEBUG_KILL_PARTICLES_)
				UE_LOGF(LogParticles, Log, "Killing Particle %4d - Next = %4d, Prev = %4d, Type = %8ls", 
					CurrentIndex, 
					TRAIL_EMITTER_GET_NEXT(TrailData->Flags),
					TRAIL_EMITTER_GET_PREV(TrailData->Flags),
					TRAIL_EMITTER_IS_ONLY(TrailData->Flags) ? TEXT("ONLY") :
					TRAIL_EMITTER_IS_START(TrailData->Flags) ? TEXT("START") :
					TRAIL_EMITTER_IS_END(TrailData->Flags) ? TEXT("END") :
					TRAIL_EMITTER_IS_MIDDLE(TrailData->Flags) ? TEXT("MIDDLE") :
					TRAIL_EMITTER_IS_DEADTRAIL(TrailData->Flags) ? TEXT("DEAD") :
					TEXT("????")
					);
#endif	//#if defined(_TRAILS_DEBUG_KILL_PARTICLES_)

				if (TRAIL_EMITTER_IS_HEAD(TrailData->Flags) || TRAIL_EMITTER_IS_ONLY(TrailData->Flags))
				{
					// Set the 'next' one in the list to the start
					int32 Next = TRAIL_EMITTER_GET_NEXT(TrailData->Flags);
					if (Next != TRAIL_EMITTER_NULL_NEXT)
					{
						DECLARE_PARTICLE_PTR(NextParticle, ParticleData + ParticleStride * Next);
						FTrailsBaseTypeDataPayload* NextTrailData = ((FTrailsBaseTypeDataPayload*)((uint8*)NextParticle + TypeDataOffset));
						if (TRAIL_EMITTER_IS_END(NextTrailData->Flags))
						{
							if (TRAIL_EMITTER_IS_START(TrailData->Flags))
							{
								NextTrailData->Flags = TRAIL_EMITTER_SET_ONLY(NextTrailData->Flags);
								SetStartIndex(NextTrailData->TrailIndex, Next);
							}
							else if (TRAIL_EMITTER_IS_DEADTRAIL(TrailData->Flags))
							{
								NextTrailData->Flags = TRAIL_EMITTER_SET_DEADTRAIL(NextTrailData->Flags);
								SetDeadIndex(NextTrailData->TrailIndex, Next);
							}
							check(TRAIL_EMITTER_GET_NEXT(NextTrailData->Flags) == TRAIL_EMITTER_NULL_NEXT);
						}
						else
						{
							if (TRAIL_EMITTER_IS_START(TrailData->Flags))
							{
								NextTrailData->Flags = TRAIL_EMITTER_SET_START(NextTrailData->Flags);
								SetStartIndex(NextTrailData->TrailIndex, Next);
							}
							else
							{
								NextTrailData->Flags = TRAIL_EMITTER_SET_DEADTRAIL(NextTrailData->Flags);
								SetDeadIndex(NextTrailData->TrailIndex, Next);
							}
						}
						NextTrailData->Flags = TRAIL_EMITTER_SET_PREV(NextTrailData->Flags, TRAIL_EMITTER_NULL_PREV);
					}
				}
				else if (TRAIL_EMITTER_IS_END(TrailData->Flags))
				{
					// See if there is a 'prev'
					int32 Prev = TRAIL_EMITTER_GET_PREV(TrailData->Flags);
					if (Prev != TRAIL_EMITTER_NULL_PREV)
					{
						DECLARE_PARTICLE_PTR(PrevParticle, ParticleData + ParticleStride * Prev);
						FTrailsBaseTypeDataPayload* PrevTrailData = ((FTrailsBaseTypeDataPayload*)((uint8*)PrevParticle + TypeDataOffset));
						if (TRAIL_EMITTER_IS_START(PrevTrailData->Flags))
						{
							PrevTrailData->Flags = TRAIL_EMITTER_SET_ONLY(PrevTrailData->Flags);
							SetStartIndex(PrevTrailData->TrailIndex, Prev);
						}
						else if (TRAIL_EMITTER_IS_DEADTRAIL(PrevTrailData->Flags))
						{
							// Nothing to do in this case.
							PrevTrailData->TriangleCount = 0;
							PrevTrailData->RenderingInterpCount = 1;
						}
						else
						{
							PrevTrailData->Flags = TRAIL_EMITTER_SET_END(PrevTrailData->Flags);
							SetEndIndex(PrevTrailData->TrailIndex, Prev);
						}
						PrevTrailData->Flags = TRAIL_EMITTER_SET_NEXT(PrevTrailData->Flags, TRAIL_EMITTER_NULL_NEXT);
					}
				}
				else if (TRAIL_EMITTER_IS_MIDDLE(TrailData->Flags))
				{
					// Break the trail? Or kill off from here to the end
					int32	Next = TRAIL_EMITTER_GET_NEXT(TrailData->Flags);
					int32	Prev = TRAIL_EMITTER_GET_PREV(TrailData->Flags);

					// Kill off the broken segment...
					if (Prev != TRAIL_EMITTER_NULL_PREV)
					{
						DECLARE_PARTICLE_PTR(PrevParticle, ParticleData + ParticleStride * Prev);
						FTrailsBaseTypeDataPayload* PrevTrailData = ((FTrailsBaseTypeDataPayload*)((uint8*)PrevParticle + TypeDataOffset));
						if (TRAIL_EMITTER_IS_START(PrevTrailData->Flags))
						{
							PrevTrailData->Flags = TRAIL_EMITTER_SET_ONLY(PrevTrailData->Flags);
							SetStartIndex(PrevTrailData->TrailIndex, Prev);
						}
						else if (TRAIL_EMITTER_IS_DEADTRAIL(PrevTrailData->Flags))
						{
							// Nothing to do in this case.
							PrevTrailData->TriangleCount = 0;
							PrevTrailData->RenderingInterpCount = 1;
						}
						else
						{
							PrevTrailData->Flags = TRAIL_EMITTER_SET_END(PrevTrailData->Flags);
							SetEndIndex(PrevTrailData->TrailIndex, Prev);
						}
						PrevTrailData->Flags = TRAIL_EMITTER_SET_NEXT(PrevTrailData->Flags, TRAIL_EMITTER_NULL_NEXT);
					}

					while (Next != TRAIL_EMITTER_NULL_NEXT)
					{
						DECLARE_PARTICLE_PTR(NextParticle, ParticleData + ParticleStride * Next);
						FTrailsBaseTypeDataPayload* NextTrailData = ((FTrailsBaseTypeDataPayload*)((uint8*)NextParticle + TypeDataOffset));
						NextTrailData->Flags = TRAIL_EMITTER_SET_FORCEKILL(NextTrailData->Flags);
						SetDeadIndex(NextTrailData->TrailIndex, Next);
						Next = TRAIL_EMITTER_GET_NEXT(NextTrailData->Flags);
						bHasForceKillParticles = true;
					}
				}
				else if (TRAIL_EMITTER_IS_FORCEKILL(TrailData->Flags))
				{
				}
				else
				{
					check(!TEXT("What the hell are you doing in here?"));
				}

				//Be certain we've broken links to this particle.
				//Some spawn paths leave zombie "forcekill" particles in the buffer so we avoid doing these. Event spawning being one.
				//These particles have malformed flags so clearing their next/prev can corrupt other particle data.
				if (!TRAIL_EMITTER_IS_FORCEKILL(TrailData->Flags))
				{
					int32 Next = TRAIL_EMITTER_GET_NEXT(TrailData->Flags);
					if (Next != TRAIL_EMITTER_NULL_NEXT)
					{
						DECLARE_PARTICLE_PTR(NextParticle, ParticleData + ParticleStride * Next);
						FTrailsBaseTypeDataPayload* NextTrailData = ((FTrailsBaseTypeDataPayload*)((uint8*)NextParticle + TypeDataOffset));
						NextTrailData->Flags = TRAIL_EMITTER_SET_PREV(NextTrailData->Flags, TRAIL_EMITTER_NULL_PREV);
					}
					int32 Prev = TRAIL_EMITTER_GET_PREV(TrailData->Flags);
					if (Prev != TRAIL_EMITTER_NULL_PREV)
					{
						DECLARE_PARTICLE_PTR(PrevParticle, ParticleData + ParticleStride * Prev);
						FTrailsBaseTypeDataPayload* PrevTrailData = ((FTrailsBaseTypeDataPayload*)((uint8*)PrevParticle + TypeDataOffset));
						PrevTrailData->Flags = TRAIL_EMITTER_SET_NEXT(PrevTrailData->Flags, TRAIL_EMITTER_NULL_NEXT);
					}
				}

				// Clear it out... just to be safe when it gets pulled back to active
				TrailData->Flags = TRAIL_EMITTER_SET_NEXT(TrailData->Flags, TRAIL_EMITTER_NULL_NEXT);
				TrailData->Flags = TRAIL_EMITTER_SET_PREV(TrailData->Flags, TRAIL_EMITTER_NULL_PREV);

				ParticleIndices[ParticleIdx] = ParticleIndices[ActiveParticles - 1];
				ParticleIndices[ActiveParticles - 1] = CurrentIndex;
				ActiveParticles--;
				SetDeadIndex(TrailData->TrailIndex, CurrentIndex);

				INC_DWORD_STAT(STAT_TrailParticlesKilled);
			}
		}

		if (bHasForceKillParticles == true)
		{
			// need to kill all these off as well...
			for (int32 ParticleIdx = ActiveParticles - 1; ParticleIdx >= 0; ParticleIdx--)
			{
				const int32 CurrentIndex = ParticleIndices[ParticleIdx];
				DECLARE_PARTICLE_PTR(Particle, ParticleData + ParticleStride * CurrentIndex);
				FTrailsBaseTypeDataPayload* TrailData = ((FTrailsBaseTypeDataPayload*)((uint8*)Particle + TypeDataOffset));
				if (TRAIL_EMITTER_IS_FORCEKILL(TrailData->Flags))
				{
					TrailData->Flags = TRAIL_EMITTER_SET_NEXT(TrailData->Flags, TRAIL_EMITTER_NULL_NEXT);
					TrailData->Flags = TRAIL_EMITTER_SET_PREV(TrailData->Flags, TRAIL_EMITTER_NULL_PREV);

					ParticleIndices[ParticleIdx] = ParticleIndices[ActiveParticles-1];
					ParticleIndices[ActiveParticles-1]	= CurrentIndex;
					ActiveParticles--;
					SetDeadIndex(TrailData->TrailIndex, CurrentIndex);
				}
			}
		}
	}
}

/**
 *	Kill the given number of particles from the end of the trail.
 *
 *	@param	InTrailIdx		The trail to kill particles in.
 *	@param	InKillCount		The number of particles to kill off.
 */
void FParticleTrailsEmitterInstance_Base::KillParticles(int32 InTrailIdx, int32 InKillCount)
{
	if (ActiveParticles)
	{
		int32 KilledCount = 0;

		{
			// Find the end particle
			FTrailsBaseTypeDataPayload* EndTrailData = nullptr;
			FBaseParticle *EndParticle = nullptr;
			int32 EndIndex;
			GetTrailEnd<FTrailsBaseTypeDataPayload>(InTrailIdx, EndIndex, EndTrailData, EndParticle);
			if (EndParticle)
			{
				if (EndTrailData && EndTrailData->TrailIndex == InTrailIdx)
				{
					while ((EndTrailData != NULL) && (KilledCount < InKillCount))
					{
						// Mark it for death...
						EndParticle->RelativeTime = 1.1f;
						KilledCount++;
						// See if there is a 'prev'
						int32 Prev = TRAIL_EMITTER_GET_PREV(EndTrailData->Flags);
						if (Prev != TRAIL_EMITTER_NULL_PREV)
						{
							EndParticle = (FBaseParticle*)(ParticleData + (ParticleStride * Prev));
							EndTrailData = ((FTrailsBaseTypeDataPayload*)((uint8*)EndParticle + TypeDataOffset));
							if (TRAIL_EMITTER_IS_START(EndTrailData->Flags))
							{
								// Don't kill the start, no matter what...
								EndTrailData = NULL;
							}
							else if (TRAIL_EMITTER_IS_DEADTRAIL(EndTrailData->Flags))
							{
								// Nothing to do in this case.
								EndTrailData->TriangleCount = 0;
								EndTrailData->RenderingInterpCount = 1;
							}
						}
					}

					if (EndTrailData == NULL)
					{
						// Force it to exit the loop...
						KilledCount = InKillCount;
					}
				}
			}
		}
		if (KilledCount > 0)
		{
			// Now use the standard KillParticles call...
			KillParticles();
		}
	}
}

/**
 *	Called when the particle system is deactivating...
 */
void FParticleTrailsEmitterInstance_Base::OnDeactivateSystem()
{
	FParticleEmitterInstance::OnDeactivateSystem();

	// Mark trails as dead if the option has been enabled...
	if (bDeadTrailsOnDeactivate == true)
	{
		for (int32 ParticleIdx = 0; ParticleIdx < ActiveParticles; ParticleIdx++)
		{
			DECLARE_PARTICLE_PTR(Particle, ParticleData + ParticleStride * ParticleIndices[ParticleIdx]);
			FBaseParticle* CurrParticle = Particle;
			FTrailsBaseTypeDataPayload*	CurrTrailData = ((FTrailsBaseTypeDataPayload*)((uint8*)Particle + TypeDataOffset));
			if (TRAIL_EMITTER_IS_ONLY(CurrTrailData->Flags) || TRAIL_EMITTER_IS_START(CurrTrailData->Flags))
			{
				CurrTrailData->Flags = TRAIL_EMITTER_SET_DEADTRAIL(CurrTrailData->Flags);
				SetDeadIndex(CurrTrailData->TrailIndex, ParticleIndices[ParticleIdx]);
			}
		}
	}
}

/**
 *	Retrieve the particle in the trail that meets the given criteria
 *
 *	@param	bSkipStartingParticle		If true, don't check the starting particle for meeting the criteria
 *	@param	InStartingFromParticle		The starting point for the search.
 *	@param	InStartingTrailData			The trail data for the starting point.
 *	@param	bPrevious					If true, search PREV entries. Search NEXT entries otherwise.
 *	@param	InGetOption					Options for defining the type of particle.
 *	@param	OutParticle					The particle that meets the criteria.
 *	@param	OutTrailData				The trail data of the particle that meets the criteria.
 *
 *	@return	bool						true if found, false if not.
 */
bool FParticleTrailsEmitterInstance_Base::GetParticleInTrail(
	bool bSkipStartingParticle,
	FBaseParticle* InStartingFromParticle,
	FTrailsBaseTypeDataPayload* InStartingTrailData,
	EGetTrailDirection InGetDirection, 
	EGetTrailParticleOption InGetOption,
	FBaseParticle*& OutParticle,
	FTrailsBaseTypeDataPayload*& OutTrailData)
{
	OutParticle = NULL;
	OutTrailData = NULL;
	if ((InStartingFromParticle == NULL) || (InStartingTrailData == NULL))
	{
		return false;
	}

	if ((InGetOption == GET_End) && (InGetDirection == GET_Prev))
	{
		// Wrong direction!
		UE_LOGF(LogParticles, Warning, "GetParticleInTrail: END particle will always be in the NEXT direction!");
	}
	if ((InGetOption == GET_Start) && (InGetDirection == GET_Next))
	{
		// Wrong direction!
		UE_LOGF(LogParticles, Warning, "GetParticleInTrail: START particle will always be in the PREV direction!");
	}

	bool bDone = false;
	FBaseParticle* CheckParticle = InStartingFromParticle;
	FTrailsBaseTypeDataPayload* CheckTrailData = InStartingTrailData;
	bool bCheckIt = !bSkipStartingParticle;
	while (!bDone)
	{
		if (bCheckIt == true)
		{
			bool bItsGood = false;
			switch (InGetOption)
			{
			case GET_Any:
				bItsGood = true;
				break;
			case GET_Spawned:
				if (CheckTrailData->bInterpolatedSpawn == false)
				{
					bItsGood = true;
				}
				break;
			case GET_Interpolated:
				if (CheckTrailData->bInterpolatedSpawn == true)
				{
					bItsGood = true;
				}
				break;
			case GET_Start:
				if (TRAIL_EMITTER_IS_START(CheckTrailData->Flags))
				{
					bItsGood = true;
				}
				break;
			case GET_End:
				if (TRAIL_EMITTER_IS_END(CheckTrailData->Flags))
				{
					bItsGood = true;
				}
				break;
			}


			if (bItsGood == true)
			{
				OutParticle = CheckParticle;
				OutTrailData = CheckTrailData;
				bDone = true;
			}
		}
		int32 Index = -1;
		if (!bDone)
		{
			// Keep looking...
			if (InGetDirection == GET_Prev)
			{
				Index = TRAIL_EMITTER_GET_PREV(CheckTrailData->Flags);
				if (Index == TRAIL_EMITTER_NULL_PREV)
				{
					Index = -1;
				}
			}
			else
			{
				Index = TRAIL_EMITTER_GET_NEXT(CheckTrailData->Flags);
				if (Index == TRAIL_EMITTER_NULL_NEXT)
				{
					Index = -1;
				}
			}
		}

		if (Index != -1)
		{
			DECLARE_PARTICLE_PTR(TempParticle, ParticleData + ParticleStride * Index);
			CheckParticle = TempParticle;
			CheckTrailData = ((FTrailsBaseTypeDataPayload*)((uint8*)CheckParticle + TypeDataOffset));
			bCheckIt = true;
		}
		else
		{
			bDone = true;
		}
	}

	return ((OutParticle != NULL) && (OutTrailData != NULL));
}

UMaterialInterface* FParticleTrailsEmitterInstance_Base::GetCurrentMaterial()
{
	UMaterialInterface* RenderMaterial = CurrentMaterial;
	if ((RenderMaterial == NULL) || (RenderMaterial->CheckMaterialUsage_Concurrent(MATUSAGE_BeamTrails) == false))
	{
		RenderMaterial = UMaterial::GetDefaultMaterial(MD_Surface);
	}
	CurrentMaterial = RenderMaterial;
	return RenderMaterial;
}

/*-----------------------------------------------------------------------------
	FParticleRibbonEmitterInstance.
-----------------------------------------------------------------------------*/
/** Constructor	*/
FParticleRibbonEmitterInstance::FParticleRibbonEmitterInstance(IParticleEmitterInstanceOwner& InComponent) :
	  FParticleTrailsEmitterInstance_Base(InComponent)
	, TrailTypeData(NULL)
	, SpawnPerUnitModule(NULL)
	, SourceModule(NULL)
	, TrailModule_Source_Offset(-1)
	, SourceActor(NULL)
	, SourceEmitter(NULL)
	, LastSelectedParticleIndex(-1)
	, HeadOnlyParticles(0)
{
	// Always want this true for ribbons...
	bEnableInactiveTimeTracking = true;
	CurrentSourcePosition.Empty();
	CurrentSourceRotation.Empty();
	CurrentSourceUp.Empty();
	CurrentSourceTangent.Empty();
	CurrentSourceTangentStrength.Empty();
	LastSourcePosition.Empty();
	LastSourceRotation.Empty();
	LastSourceUp.Empty();
	LastSourceTangent.Empty();
	LastSourceTangentStrength.Empty();
	SourceOffsets.Empty();
	SourceIndices.Empty();
	SourceTimes.Empty();
	LastSourceTimes.Empty();
	CurrentLifetimes.Empty();
}

/** Destructor	*/
FParticleRibbonEmitterInstance::~FParticleRibbonEmitterInstance()
{
}

void FParticleRibbonEmitterInstance::InitParameters(UParticleEmitter* InTemplate)
{
	FParticleTrailsEmitterInstance_Base::InitParameters(InTemplate);

	// We don't support LOD on trails
	UParticleLODLevel* LODLevel	= InTemplate->GetLODLevel(0);
	check(LODLevel);
	TrailTypeData	= CastChecked<UParticleModuleTypeDataRibbon>(LODLevel->TypeDataModule);
	check(TrailTypeData);

	// Always have at least one trail
	if (TrailTypeData->MaxTrailCount <= 0)
	{
		TrailTypeData->MaxTrailCount = 1;
	}

	bDeadTrailsOnDeactivate = TrailTypeData->bDeadTrailsOnDeactivate;

	MaxTrailCount = TrailTypeData->MaxTrailCount;
	TrailSpawnTimes.Empty(MaxTrailCount);
	TrailSpawnTimes.AddZeroed(MaxTrailCount);
	CurrentSourcePosition.Empty(MaxTrailCount);
	CurrentSourcePosition.AddZeroed(MaxTrailCount);
	CurrentSourceRotation.Empty(MaxTrailCount);
	CurrentSourceRotation.AddZeroed(MaxTrailCount);
	CurrentSourceUp.Empty(MaxTrailCount);
	CurrentSourceUp.AddZeroed(MaxTrailCount);
	CurrentSourceTangent.Empty(MaxTrailCount);
	CurrentSourceTangent.AddZeroed(MaxTrailCount);
	CurrentSourceTangentStrength.Empty(MaxTrailCount);
	CurrentSourceTangentStrength.AddZeroed(MaxTrailCount);
	LastSourcePosition.Empty(MaxTrailCount);
	LastSourcePosition.AddZeroed(MaxTrailCount);
	LastSourceRotation.Empty(MaxTrailCount);
	LastSourceRotation.AddZeroed(MaxTrailCount);
	LastSourceUp.Empty(MaxTrailCount);
	LastSourceUp.AddZeroed(MaxTrailCount);
	LastSourceTangent.Empty(MaxTrailCount);
	LastSourceTangent.AddZeroed(MaxTrailCount);
	LastSourceTangentStrength.Empty(MaxTrailCount);
	LastSourceTangentStrength.AddZeroed(MaxTrailCount);
	SourceDistanceTraveled.Empty(MaxTrailCount);
	SourceDistanceTraveled.AddZeroed(MaxTrailCount);
	TiledUDistanceTraveled.Empty(MaxTrailCount);
	TiledUDistanceTraveled.AddZeroed(MaxTrailCount);
	SourceOffsets.Empty(MaxTrailCount);
	SourceOffsets.AddZeroed(MaxTrailCount);
	SourceIndices.Empty(MaxTrailCount);
	SourceIndices.AddZeroed(MaxTrailCount);
	FMemory::Memset(SourceIndices.GetData(), 0xff, MaxTrailCount * sizeof(int32));
	SourceTimes.Empty(MaxTrailCount);
	SourceTimes.AddZeroed(MaxTrailCount);
	LastSourceTimes.Empty(MaxTrailCount);
	LastSourceTimes.AddZeroed(MaxTrailCount);
	CurrentLifetimes.Empty(MaxTrailCount);
	CurrentLifetimes.AddZeroed(MaxTrailCount);
	CurrentSizes.Empty(MaxTrailCount);
	CurrentSizes.AddZeroed(MaxTrailCount);

	//
	VertexCount = 0;
	TriangleCount = 0;

	// Resolve any actors...
	ResolveSource();
}

void TrailsBase_CalculateTangent(
	FBaseParticle* InPrevParticle, FRibbonTypeDataPayload* InPrevTrailData, 
	FBaseParticle* InNextParticle, FRibbonTypeDataPayload* InNextTrailData,
	float InCurrNextDelta, FRibbonTypeDataPayload* InOutCurrTrailData)
{
	// Recalculate the current tangent...
	// Calculate the new tangent from the previous and next position...
	// LastSourcePosition will be that position of the first particle that will be spawned this frame
	FVector PositionDelta = InPrevParticle->Location - InNextParticle->Location;
	float TimeDelta = InPrevTrailData->SpawnTime - InNextTrailData->SpawnTime;
	// Disabling the check as it apparently is causing problems 
	//check(TimeDelta >= 0.0f);

	TimeDelta = (TimeDelta == 0.0f) ? 0.0032f : FMath::Abs(TimeDelta);

	FVector NewTangent = (PositionDelta / TimeDelta);

	NewTangent *= InCurrNextDelta;
	NewTangent *= (1.0f / InOutCurrTrailData->SpawnedTessellationPoints);

		InOutCurrTrailData->Tangent = (FVector3f)NewTangent;
}

/**
 *	Tick sub-function that handles recalculation of tangents
 *
 *	@param	DeltaTime			The current time slice
 *	@param	CurrentLODLevel		The current LOD level for the instance
 */
void FParticleRibbonEmitterInstance::Tick_RecalculateTangents(float DeltaTime, UParticleLODLevel* InCurrentLODLevel)
{
	if (TrailTypeData->bTangentRecalculationEveryFrame == true)
	{
		for (int32 TrailIdx = 0; TrailIdx < MaxTrailCount; TrailIdx++)
		{
			// Find the Start particle of the current trail...
			FBaseParticle* StartParticle = NULL;
			FRibbonTypeDataPayload* StartTrailData = NULL;
			int32 StartIndex = -1;
			GetTrailStart<FRibbonTypeDataPayload>(TrailIdx, StartIndex, StartTrailData, StartParticle);

			// Recalculate tangents at each particle to properly handle moving particles...
			if ((StartParticle != NULL) && (TRAIL_EMITTER_IS_ONLY(StartTrailData->Flags) == 0))
			{
				// For trails, particles go:
				//     START, next, next, ..., END
				// Coming from the end,
				//     END, prev, prev, ..., START
				FBaseParticle* PrevParticle = StartParticle;
				FRibbonTypeDataPayload* PrevTrailData = StartTrailData;
				FBaseParticle* CurrParticle = NULL; 
				FRibbonTypeDataPayload* CurrTrailData = NULL;
				FBaseParticle* NextParticle = NULL;
				FTrailsBaseTypeDataPayload* TempPayload = NULL;
				FRibbonTypeDataPayload* NextTrailData = NULL;

				GetParticleInTrail(true, PrevParticle, PrevTrailData, GET_Next, GET_Any, CurrParticle, TempPayload);
				CurrTrailData = (FRibbonTypeDataPayload*)(TempPayload);

				// Deal with the start particle...
				if (CurrParticle != NULL)
				{
					TrailsBase_CalculateTangent(PrevParticle, PrevTrailData, CurrParticle, CurrTrailData, 
						(PrevTrailData->SpawnTime - CurrTrailData->SpawnTime), 
						PrevTrailData);
				}

				while (CurrParticle != NULL)
				{
					// Grab the next particle in the trail...
					GetParticleInTrail(true, CurrParticle, CurrTrailData, GET_Next, GET_Any, NextParticle, TempPayload);
					NextTrailData = (FRibbonTypeDataPayload*)(TempPayload);

					check(CurrParticle != PrevParticle);
					check(CurrParticle != NextParticle);

					if (NextParticle != NULL)
					{
						TrailsBase_CalculateTangent(PrevParticle, PrevTrailData, NextParticle, NextTrailData, 
							(CurrTrailData->SpawnTime - NextTrailData->SpawnTime),
							CurrTrailData);
					}
					else
					{
						// The start particle... should we recalc w/ the current source position???
						TrailsBase_CalculateTangent(PrevParticle, PrevTrailData, CurrParticle, CurrTrailData, 
							(PrevTrailData->SpawnTime - CurrTrailData->SpawnTime),
							CurrTrailData);
					}

					// Move up the chain...
					PrevParticle = CurrParticle;
					PrevTrailData = CurrTrailData;
					CurrParticle = NextParticle;
					CurrTrailData = NextTrailData;
				}
			}
		}
	}
}

bool FParticleRibbonEmitterInstance::GetSpawnPerUnitAmount(float DeltaTime, int32 InTrailIdx, int32& OutCount, float& OutRate)
{
	check(CurrentSourcePosition.Num() > InTrailIdx);
	check(LastSourcePosition.Num() > InTrailIdx);
	check(SpawnPerUnitModule != NULL);

	if (SpawnPerUnitModule && SpawnPerUnitModule->bEnabled)
	{
		bool bMoved = false;
		float NewTravelLeftover = 0.0f;
		float ParticlesPerUnit = SpawnPerUnitModule->SpawnPerUnit.GetValue(EmitterTime, Component.GetDistributionData()) / SpawnPerUnitModule->UnitScalar;
		// Allow for PPU of 0.0f to allow for 'turning off' an emitter when moving
		if (ParticlesPerUnit >= 0.0f)
		{
			float LeftoverTravel = SourceDistanceTraveled[InTrailIdx];
			// Calculate movement delta over last frame, include previous remaining delta
			FVector TravelDirection = CurrentSourcePosition[InTrailIdx] - LastSourcePosition[InTrailIdx];
			// Calculate distance traveled
			float TravelDistance = TravelDirection.Size();
			if (((SpawnPerUnitModule->MaxFrameDistance > 0.0f) && (TravelDistance > SpawnPerUnitModule->MaxFrameDistance)) ||
				(TravelDistance > HALF_WORLD_MAX))
			{
				// Clear it out!
				uint8* InstData = GetModuleInstanceData(SpawnPerUnitModule);
				FParticleSpawnPerUnitInstancePayload* SPUPayload = NULL;
				SPUPayload = (FParticleSpawnPerUnitInstancePayload*)InstData;

				//@todo. Need to 'shift' the start point closer so we can still spawn...
				TravelDistance = 0.0f;
				SPUPayload->CurrentDistanceTravelled = 0.0f;
				LastSourcePosition[InTrailIdx] = CurrentSourcePosition[InTrailIdx];
			}

			// Check the change in tangent from last to this...
			float CheckTangent = 0.0f;
			if (TrailTypeData->TangentSpawningScalar > 0.0f)
			{
				float ElapsedTime = RunningTime;//SecondsSinceCreation;
				if (ActiveParticles == 0)
				{
					if (ElapsedTime == 0)
					{
						ElapsedTime = UE_KINDA_SMALL_NUMBER;
					}
					CurrentSourcePosition[InTrailIdx].DiagnosticCheckNaN();
					LastSourcePosition[InTrailIdx].DiagnosticCheckNaN();

					LastSourceTangent[InTrailIdx] = (CurrentSourcePosition[InTrailIdx] - LastSourcePosition[InTrailIdx]) / ElapsedTime;
				}

				float CurrTangentDivisor = (ElapsedTime - TrailSpawnTimes[InTrailIdx]);
				if (CurrTangentDivisor == 0)
				{
					CurrTangentDivisor = UE_KINDA_SMALL_NUMBER;
				}
				FVector CurrTangent = TravelDirection / CurrTangentDivisor;
				CurrTangent.Normalize();
				FVector PrevTangent = LastSourceTangent[InTrailIdx];
				PrevTangent.Normalize();
				CheckTangent = (CurrTangent | PrevTangent);
				// Map the tangent difference to [0..1] for [0..180]
				//  1.0 = parallel    --> -1 = 0
				//  0.0 = orthogonal  --> -1 = -1 --> * -0.5 = 0.5
				// -1.0 = oppositedir --> -1 = -2 --> * -0.5 = 1.0
				CheckTangent = (CheckTangent - 1.0f) * -0.5f;
			}

			if (TravelDistance > 0.0f)
			{
				if (TravelDistance > (SpawnPerUnitModule->MovementTolerance * SpawnPerUnitModule->UnitScalar))
				{
					bMoved = true;
				}

				// Normalize direction for use later
				TravelDirection.Normalize();

				// Calculate number of particles to emit
				float NewLeftover = (TravelDistance + LeftoverTravel) * ParticlesPerUnit;

				NewLeftover += CheckTangent * TrailTypeData->TangentSpawningScalar;

				const float InvDeltaTime = DeltaTime > 0.0f ? 1.0f / DeltaTime : 0.0f;
				OutCount = (TrailTypeData->bSpawnInitialParticle && !ActiveParticles && (NewLeftover < 1.0f))? 1: FMath::FloorToInt(NewLeftover);
				OutRate = OutCount * InvDeltaTime;
				NewTravelLeftover = (TravelDistance + LeftoverTravel) - (OutCount * SpawnPerUnitModule->UnitScalar);
				SourceDistanceTraveled[InTrailIdx] = FMath::Max<float>(0.0f, NewTravelLeftover);
			}
			else
			{
				OutCount = 0;
				OutRate = 0.0f;
			}
		}
		else
		{
			OutCount = 0;
			OutRate = 0.0f;
		}

		if (SpawnPerUnitModule->bIgnoreSpawnRateWhenMoving == true)
		{
			if (bMoved == true)
			{
				return false;
			}
			return true;
		}
	}

	return SpawnPerUnitModule->bProcessSpawnRate;
}

bool FParticleTrailsEmitterInstance_Base::AddParticleHelper(int32 InTrailIdx,
	int32 StartParticleIndex, FTrailsBaseTypeDataPayload* StartTrailData,
	int32 ParticleIndex, FTrailsBaseTypeDataPayload* TrailData
	)
{
	bool bAddedParticle = false;

	TrailData->TrailIndex = InTrailIdx;
	if (TRAIL_EMITTER_IS_ONLY(StartTrailData->Flags))
	{
		StartTrailData->Flags	= TRAIL_EMITTER_SET_END(StartTrailData->Flags);
		StartTrailData->Flags	= TRAIL_EMITTER_SET_NEXT(StartTrailData->Flags, TRAIL_EMITTER_NULL_NEXT);
		StartTrailData->Flags	= TRAIL_EMITTER_SET_PREV(StartTrailData->Flags, ParticleIndex);

		// we're adding an end particle here, after having only a start
		SetEndIndex(StartTrailData->TrailIndex, StartParticleIndex);


#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
		if (TrailData->SpawnTime < StartTrailData->SpawnTime)
		{
			UE_LOGF(LogParticles, Log, "BAD SPAWN TIME! Curr %8.6f (%9ls), Start %8.6f (%9ls), %ls (%ls)", 
				TrailData->SpawnTime, 
				TrailData->bMovementSpawned ? TEXT("MOVESPAWN") : TEXT("SPAWNRATE"),
				StartTrailData->SpawnTime,
				StartTrailData->bMovementSpawned ? TEXT("MOVESPAWN") : TEXT("SPAWNRATE"),
				(Component.GetTemplate() ? *(Component.GetTemplate()->GetPathName()) : TEXT("*** No Template")),
				*Component.GetPathName());
		}
#endif

		// Now, 'join' them
		TrailData->Flags		= TRAIL_EMITTER_SET_PREV(TrailData->Flags, TRAIL_EMITTER_NULL_PREV);
		TrailData->Flags		= TRAIL_EMITTER_SET_NEXT(TrailData->Flags, StartParticleIndex);
		TrailData->Flags		= TRAIL_EMITTER_SET_START(TrailData->Flags);

		SetStartIndex(TrailData->TrailIndex, ParticleIndex);

		bAddedParticle = true;
	}
	else
	{
		// It better be the start!!!
		check(TRAIL_EMITTER_IS_START(StartTrailData->Flags));
		check(TRAIL_EMITTER_GET_NEXT(StartTrailData->Flags) != TRAIL_EMITTER_NULL_NEXT);

		StartTrailData->Flags	= TRAIL_EMITTER_SET_MIDDLE(StartTrailData->Flags);
		StartTrailData->Flags	= TRAIL_EMITTER_SET_PREV(StartTrailData->Flags, ParticleIndex);
		ClearIndices(StartTrailData->TrailIndex, StartParticleIndex);

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
		if (TrailData->SpawnTime < StartTrailData->SpawnTime)
		{
			UE_LOGF(LogParticles, Log, "BAD SPAWN TIME! Curr %8.6f (%9ls), Start %8.6f (%9ls), %ls (%ls)", 
				TrailData->SpawnTime, 
				TrailData->bMovementSpawned ? TEXT("MOVESPAWN") : TEXT("SPAWNRATE"),
				StartTrailData->SpawnTime,
				StartTrailData->bMovementSpawned ? TEXT("MOVESPAWN") : TEXT("SPAWNRATE"),
				(Component.GetTemplate() ? *(Component.GetTemplate()->GetPathName()) : TEXT("*** No Template")),
				*Component.GetPathName());
		}
#endif

		// Now, 'join' them
		TrailData->Flags		= TRAIL_EMITTER_SET_PREV(TrailData->Flags, TRAIL_EMITTER_NULL_PREV);
		TrailData->Flags		= TRAIL_EMITTER_SET_NEXT(TrailData->Flags, StartParticleIndex);
		TrailData->Flags		= TRAIL_EMITTER_SET_START(TrailData->Flags);

		SetStartIndex(TrailData->TrailIndex, ParticleIndex);

		//SourceDistanceTravelled(TrailData->TrailIndex) += SourceDistanceTravelled(CheckTrailData->TrailIndex);

		bAddedParticle = true;
	}

	return bAddedParticle;
}

/**
 *	Get the lifetime and size for a particle being added to the given trail
 *	
 *	@param	InTrailIdx				The index of the trail the particle is being added to
 *	@param	InParticle				The particle that is being added
 *	@param	bInNoLivingParticles	true if there are no particles in the trail, false if there already are
 *	@param	OutOneOverMaxLifetime	The OneOverMaxLifetime value to use for the particle
 *	@param	OutSize					The Size value to use for the particle
 */
void FParticleRibbonEmitterInstance::GetParticleLifetimeAndSize(int32 InTrailIdx, const FBaseParticle* InParticle, bool bInNoLivingParticles, float& OutOneOverMaxLifetime, float& OutSize)
{
	if (bInNoLivingParticles == true)
	{
		UParticleLODLevel* LODLevel = SpriteTemplate->LODLevels[0];
		check(LODLevel);

		// Find the lifetime module
		float CurrLifetime = 0.0f;
		for (int32 ModuleIdx = 0; ModuleIdx < LODLevel->SpawnModules.Num(); ModuleIdx++)
		{
			UParticleModuleLifetime* LifetimeModule = Cast<UParticleModuleLifetime>(LODLevel->SpawnModules[ModuleIdx]);
			if (LifetimeModule != NULL)
			{
				float MaxLifetime = LifetimeModule->GetLifetimeValue({*this}, EmitterTime, Component.GetDistributionData());
				if (CurrLifetime > 0.f)
				{
					// Another module already modified lifetime.
					CurrLifetime = 1.f / (MaxLifetime + (1.f / CurrLifetime));
				}
				else
				{
					// First module to modify lifetime.
					CurrLifetime = (MaxLifetime > 0.f) ? (1.f / MaxLifetime) : 0.f;
				}

				break;	// consider only the first lifetime module
			}
		}
		if (CurrLifetime == 0.0f)
		{
			// We can't allow this...
			CurrLifetime = 1.0f;
		}

		if ((1.0f / CurrLifetime) < 0.001f)
		{
			CurrLifetime = 1.f / 0.001f;
		}

		CurrentLifetimes[InTrailIdx] = CurrLifetime;
		CurrentSizes[InTrailIdx] = InParticle->Size.X;
	}
	OutOneOverMaxLifetime = CurrentLifetimes[InTrailIdx];
	OutSize = CurrentSizes[InTrailIdx];
}

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
// Critical section used to display a message when particles are killed before max allowed count is reachen.
// For instance, when (HighLODLevel->PeakActiveParticles > TrailTypeData->MaxParticleInTrailCount)
static FCriticalSection GMaxParticleInTrailCountWarningCS;
#endif

float FParticleRibbonEmitterInstance::Spawn(float DeltaTime)
{
	bool bProcessSpawnRate = Spawn_Source(DeltaTime);
	if (bProcessSpawnRate == false)
	{
		return SpawnFraction;
	}

	UParticleLODLevel* LODLevel = SpriteTemplate->LODLevels[0];
	check(LODLevel);
	check(LODLevel->RequiredModule);

	// Iterate over each trail
	int32 TrailIdx = 0;

	float MovementSpawnRate = 0.0f;
	int32 MovementSpawnCount = 0;
	float SpawnRate = 0.0f;
	int32 SpawnCount = 0;
	int32 BurstCount = 0;
	float OldLeftover = SpawnFraction;
	// For now, we are not supporting bursts on trails...
	bool bProcessBurstList = false;

	// Figure out spawn rate for this tick.
	if (bProcessSpawnRate)
	{
		float RateScale = LODLevel->SpawnModule->RateScale.GetValue(EmitterTime, Component.GetDistributionData()) * LODLevel->SpawnModule->GetGlobalRateScale();
		float QualityMult = SpriteTemplate->GetQualityLevelSpawnRateMult();
		SpawnRate += LODLevel->SpawnModule->Rate.GetValue(EmitterTime, Component.GetDistributionData()) * FMath::Clamp<float>(QualityMult, 0.0f, 1.0);
	}

	// Take Bursts into account as well...
	if (bProcessBurstList)
	{
		int32 Burst = 0;
		float BurstTime = GetCurrentBurstRateOffset(DeltaTime, Burst);
		BurstCount += Burst;
	}

	const int32 LocalMaxParticleInTrailCount = TrailTypeData->MaxParticleInTrailCount;
	float SafetyLeftover = OldLeftover;
	float NewLeftover = OldLeftover + DeltaTime * SpawnRate;
	int32 SpawnNumber	= FMath::FloorToInt(NewLeftover);
	float SliceIncrement = (SpawnRate > 0.0f) ? (1.f / SpawnRate) : 0.0f;
	float SpawnStartTime = DeltaTime + OldLeftover * SliceIncrement - SliceIncrement;
	SpawnFraction = NewLeftover - SpawnNumber;

	int32 TotalCount = MovementSpawnCount + SpawnNumber + BurstCount;
	bool bNoLivingParticles = (ActiveParticles == 0);

	//@todo. Don't allow more than TrailCount trails...
	if (LocalMaxParticleInTrailCount > 0)
	{
		int32 KillCount = (TotalCount + ActiveParticles) - LocalMaxParticleInTrailCount;
		if (KillCount > 0)
		{
			KillParticles(TrailIdx, KillCount);
		}

		// Don't allow the spawning of more particles than allowed...
		TotalCount = FMath::Max<int32>(TotalCount,LocalMaxParticleInTrailCount);
	}

	// Handle growing arrays.
	bool bProcessSpawn = true;
	int32 NewCount = ActiveParticles + TotalCount;
	if (NewCount >= MaxActiveParticles)
	{
		if (DeltaTime < 0.25f)
		{
			bProcessSpawn = Resize(NewCount + FMath::TruncToInt(FMath::Sqrt((float)NewCount)) + 1);
		}
		else
		{
			bProcessSpawn = Resize((NewCount + FMath::TruncToInt(FMath::Sqrt((float)NewCount)) + 1), false);
		}
	}

	if (bProcessSpawn == false)
	{
		return SafetyLeftover;
	}

	// Find the start particle of the current trail...
	FBaseParticle* StartParticle = NULL;
	int32 StartIndex = -1;
	FRibbonTypeDataPayload* StartTrailData = NULL;
	GetTrailStart<FRibbonTypeDataPayload>(TrailIdx, StartIndex, StartTrailData, StartParticle);

	bNoLivingParticles = (StartParticle == NULL);
	bool bTilingTrail = !FMath::IsNearlyZero(TrailTypeData->TilingDistance);

	FParticleEventInstancePayload* EventPayload = NULL;
	if (LODLevel->EventGenerator)
	{
		EventPayload = (FParticleEventInstancePayload*)GetModuleInstanceData(LODLevel->EventGenerator);
		if (EventPayload && !EventPayload->bSpawnEventsPresent && !EventPayload->bBurstEventsPresent)
		{
			EventPayload = NULL;
		}
	}

	float ElapsedTime = RunningTime;//SecondsSinceCreation;
	// Do we have SpawnRate driven spawning?
	if ((SpawnRate > 0.0f) && (SpawnNumber > 0))
	{
		float Increment = (SpawnRate > 0.0f) ? (1.f / SpawnRate) : 0.0f;
		float StartTime = DeltaTime + OldLeftover * Increment - Increment;

		// Spawn particles.
		// NOTE: SpawnRate assumes that the ParticleSystemComponent is the 'source'
		FVector CurrentUp;
		if (TrailTypeData->RenderAxis == Trails_SourceUp)
		{
			CurrentUp = Component.GetComponentTransform().GetScaledAxis( EAxis::Z );
		}
		else
		{
			CurrentUp = FVector(0.0f, 0.0f, 1.0f);
		}

		float InvCount = 1.0f / SpawnNumber;

		for (int32 SpawnIdx = 0; SpawnIdx < SpawnNumber; SpawnIdx++)
		{
			check(ActiveParticles <= MaxActiveParticles);
			int32 ParticleIndex = ParticleIndices[ActiveParticles];
			DECLARE_PARTICLE_PTR(Particle, ParticleData + ParticleStride * ParticleIndex);
			FRibbonTypeDataPayload* TrailData = ((FRibbonTypeDataPayload*)((uint8*)Particle + TypeDataOffset));

			float SpawnTime = StartTime - SpawnIdx * Increment;
			float TimeStep = FMath::Clamp<float>(InvCount * (SpawnIdx + 1), 0.0f, 1.0f);
			float StoredSpawnTime = DeltaTime * TimeStep;

			PreSpawn(Particle, Location, FVector::ZeroVector);
			SetDeadIndex(TrailData->TrailIndex, ParticleIndex);
			if (LODLevel->TypeDataModule)
			{
				LODLevel->TypeDataModule->Spawn({ *this, TypeDataOffset, SpawnTime, Particle });
			}

			for (int32 ModuleIndex = 0; ModuleIndex < LODLevel->SpawnModules.Num(); ModuleIndex++)
			{
				UParticleModule* SpawnModule	= LODLevel->SpawnModules[ModuleIndex];
				if (SpawnModule->bEnabled)
				{
					UParticleModule* OffsetModule	= LODLevel->SpawnModules[ModuleIndex];
					SpawnModule->Spawn({ *this, (int32)GetModuleDataOffset(OffsetModule), SpawnTime, Particle });
				}
			}
			PostSpawn(Particle, 1.f - float(SpawnIdx + 1) / float(SpawnNumber), SpawnTime);

			GetParticleLifetimeAndSize(TrailIdx, Particle, bNoLivingParticles, Particle->OneOverMaxLifetime, Particle->Size.X);
			Particle->RelativeTime = SpawnTime * Particle->OneOverMaxLifetime;
			Particle->Size.Y = Particle->Size.X;
			Particle->BaseSize = Particle->Size;

			if (EventPayload)
			{
				LODLevel->EventGenerator->HandleParticleSpawned(this, EventPayload, Particle);
			}

			// Trail specific...
			// Clear the next and previous - just to be safe
			TrailData->Flags = TRAIL_EMITTER_SET_NEXT(TrailData->Flags, TRAIL_EMITTER_NULL_NEXT);
			TrailData->Flags = TRAIL_EMITTER_SET_PREV(TrailData->Flags, TRAIL_EMITTER_NULL_PREV);
			// Set the trail-specific data on this particle
			TrailData->TrailIndex = TrailIdx;
			TrailData->Tangent = -Particle->Velocity * DeltaTime;
			TrailData->SpawnTime = ElapsedTime + StoredSpawnTime;
			TrailData->SpawnDelta = SpawnIdx * Increment;
			// Set the location and up vectors
			TrailData->Up = (FVector3f)CurrentUp;

			TrailData->bMovementSpawned = false;

			// If this is the true spawned particle, store off the spawn interpolated count
			TrailData->bInterpolatedSpawn = false; 
			TrailData->SpawnedTessellationPoints = 1;

			bool bAddedParticle = false;
			// Determine which trail to attach to
			if (bNoLivingParticles)
			{
				// These are the first particles!
				// Tag it as the 'only'
				TrailData->Flags = TRAIL_EMITTER_SET_ONLY(TrailData->Flags);
				TiledUDistanceTraveled[TrailIdx] = 0.0f;
				TrailData->TiledU = 0.0f;
				bNoLivingParticles	= false;
				bAddedParticle		= true;
				SetStartIndex(TrailData->TrailIndex, ParticleIndex);
			}
			else if (StartParticle)
			{
				bAddedParticle = AddParticleHelper(TrailIdx, 
					StartIndex, (FTrailsBaseTypeDataPayload*)StartTrailData, 
					ParticleIndex, (FTrailsBaseTypeDataPayload*)TrailData
					);
			}

			if (bAddedParticle)
			{
				if (bTilingTrail == true)
				{
					if (StartParticle == NULL)
					{
						TrailData->TiledU = 0.0f;
					}
					else
					{
						FVector PositionDelta = Particle->Location - StartParticle->Location;
						TiledUDistanceTraveled[TrailIdx] += PositionDelta.Size();
						TrailData->TiledU = TiledUDistanceTraveled[TrailIdx] / TrailTypeData->TilingDistance;
						//@todo. Is there going to be a problem when distance gets REALLY high?
					}
				}

				StartParticle = Particle;
				StartIndex = ParticleIndex;
				StartTrailData = TrailData;

				ActiveParticles++;

				if (StartTrailData->Tangent.IsNearlyZero())
				{
					FBaseParticle* NextSpawnedParticle = NULL;
					FRibbonTypeDataPayload* NextSpawnedTrailData = NULL;
					FTrailsBaseTypeDataPayload* TempPayload = NULL;
					GetParticleInTrail(true, StartParticle, StartTrailData, GET_Next, GET_Spawned, NextSpawnedParticle, TempPayload);
					NextSpawnedTrailData = (FRibbonTypeDataPayload*)(TempPayload);
					if (NextSpawnedParticle != NULL)
					{
						FVector PositionDelta = (StartParticle->Location - NextSpawnedParticle->Location);
						float TimeDelta = StartTrailData->SpawnTime - NextSpawnedTrailData->SpawnTime;
						StartTrailData->Tangent = FVector3f(PositionDelta / TimeDelta);
					}
				}

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
				if ((int32)ActiveParticles > LocalMaxParticleInTrailCount)
				{
					if (Component.HasWorld())
					{
						const FString TemplateName = Component.GetTemplate() ? *Component.GetTemplate()->GetName() : TEXT("No template");
						GMaxParticleInTrailCountWarningCS.Lock();
						static TMap<FString, int32> TemplateMaxCountWarnings;
						int32& MaxCount = TemplateMaxCountWarnings.FindOrAdd(TemplateName);
						GMaxParticleInTrailCountWarningCS.Unlock();
						if (MaxCount < ActiveParticles)
						{
							MaxCount = ActiveParticles;
							FString ErrorMessage = FString::Printf(TEXT("Ribbon with too many particles: %5d vs. %5d, %s"), ActiveParticles, LocalMaxParticleInTrailCount, *TemplateName);
							UE_LOGF(LogParticles, Log, "%ls", *ErrorMessage);
						}
					}
				}
#endif	//#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
				INC_DWORD_STAT(STAT_TrailParticlesSpawned);

				if ((TrailTypeData->bEnablePreviousTangentRecalculation == true)
					&& (TrailTypeData->bTangentRecalculationEveryFrame == false))
				{
					// Find the 2 next SPAWNED particles in the trail (not interpolated).
					// If there are 2, then the one about to be spawned will make a chain of 3 
					// giving us data to better calculate the middle ones tangent.
					// After doing so, we must also recalculate the tangents of the interpolated
					// particles in the chain.

					// The most recent spawned particle in the trail...
					FBaseParticle* NextSpawnedParticle = NULL;
					FRibbonTypeDataPayload* NextSpawnedTrailData = NULL;
					// The second most recent spawned particle in the trail...
					FBaseParticle* NextNextSpawnedParticle = NULL;
					FRibbonTypeDataPayload* NextNextSpawnedTrailData = NULL;

					FTrailsBaseTypeDataPayload* TempPayload = NULL;

					// Grab the latest two spawned particles in the trail
					GetParticleInTrail(true, StartParticle, StartTrailData, GET_Next, GET_Spawned, NextSpawnedParticle, TempPayload);
					NextSpawnedTrailData = (FRibbonTypeDataPayload*)(TempPayload);
					GetParticleInTrail(true, NextSpawnedParticle, NextSpawnedTrailData, GET_Next, GET_Spawned, NextNextSpawnedParticle, TempPayload);
					NextNextSpawnedTrailData = (FRibbonTypeDataPayload*)(TempPayload);

					if (NextSpawnedParticle != NULL)
					{
						FVector NewTangent;
						if (NextNextSpawnedParticle != NULL)
						{
							// Calculate the new tangent from the previous and next position...
							// LastSourcePosition will be that position of the first particle that will be spawned this frame
							FVector PositionDelta = (StartParticle->Location - NextNextSpawnedParticle->Location);
							float TimeDelta = StartTrailData->SpawnTime - NextNextSpawnedTrailData->SpawnTime;
							NewTangent = PositionDelta / TimeDelta;
							NextSpawnedTrailData->Tangent = (FVector3f)NewTangent;
						}
		 				else //if (NextNextSpawnedParticle == NULL)
		 				{
		 					// This is the second spawned particle in a trail...
		 					// Calculate the new tangent from the previous and next position...
		 					// LastSourcePosition will be that position of the first particle that will be spawned this frame
		 					FVector PositionDelta = (StartParticle->Location - NextSpawnedParticle->Location);
		 					float TimeDelta = StartTrailData->SpawnTime - NextSpawnedTrailData->SpawnTime;
		 					NewTangent = PositionDelta / TimeDelta;
		 					NextSpawnedTrailData->Tangent = (FVector3f)NewTangent;
		 				}
					}
				}

				TrailSpawnTimes[0] = TrailData->SpawnTime;
			}
			else
			{
				check(TEXT("Failed to add particle to trail!!!!"));
			}

			INC_DWORD_STAT(STAT_SpriteParticlesSpawned);
		}
	}
	INC_DWORD_STAT_BY(STAT_TrailParticles, ActiveParticles);


//TickCount++;
	return SpawnFraction;
}

/**
 *	Spawn source-based ribbon particles.
 *
 *	@param	DeltaTime			The current time slice
 *
 *	@return	bool				true if SpawnRate should be processed.
 */
bool FParticleRibbonEmitterInstance::Spawn_Source(float DeltaTime)
{
	bool bProcessSpawnRate = true;
	UParticleLODLevel* LODLevel = SpriteTemplate->LODLevels[0];
	check(LODLevel);
	check(LODLevel->RequiredModule);

	const int32 LocalMaxParticleInTrailCount = TrailTypeData->MaxParticleInTrailCount;
	// Iterate over each trail
	for (int32 TrailIdx = 0; TrailIdx < MaxTrailCount; TrailIdx++)
	{
		float MovementSpawnRate = 0.0f;
		int32 MovementSpawnCount = 0;

		// Process the SpawnPerUnit, if present.
		if ((SpawnPerUnitModule != NULL) && (SpawnPerUnitModule->bEnabled == true))
		{
			// We are hijacking the settings from this - not using it to calculate the value
			// Update the spawn rate
			int32 Number = 0;
			float Rate = 0.0f;
			bProcessSpawnRate = GetSpawnPerUnitAmount(DeltaTime, TrailIdx, Number, Rate);
			MovementSpawnCount += Number;
			MovementSpawnRate += Rate;
		}

		// Do the resize stuff here!!!!!!!!!!!!!!!!!!!
		// Determine if no particles are alive
		bool bNoLivingParticles = (ActiveParticles == 0);

		// Don't allow more than TrailCount trails...
		if (LocalMaxParticleInTrailCount > 0)
		{
			int32 KillCount = (MovementSpawnCount + ActiveParticles) - LocalMaxParticleInTrailCount;
			if (KillCount > 0)
			{
				KillParticles(TrailIdx, KillCount);
			}

			if ((MovementSpawnCount + ActiveParticles) > LocalMaxParticleInTrailCount)
			{
				// We kill all the ones we could... so now we have to fall back to clamping
				MovementSpawnCount = LocalMaxParticleInTrailCount - ActiveParticles;
				if (MovementSpawnCount < 0)
				{
					MovementSpawnCount = 0;
				}
			}
		}

		// Handle growing arrays.
		bool bProcessSpawn = true;
		int32 NewCount = ActiveParticles + MovementSpawnCount;
		if (NewCount >= MaxActiveParticles)
		{
			if (DeltaTime < 0.25f)
			{
				bProcessSpawn = Resize(NewCount + FMath::TruncToInt(FMath::Sqrt((float)NewCount)) + 1);
			}
			else
			{
				bProcessSpawn = Resize((NewCount + FMath::TruncToInt(FMath::Sqrt((float)NewCount)) + 1), false);
			}
		}

		if (bProcessSpawn == false)
		{
			continue;
		}




		// Find the start particle of the current trail...
		FBaseParticle* StartParticle = NULL;
		int32 StartIndex = -1;
		FRibbonTypeDataPayload* StartTrailData = NULL;

		// temporarily not using index cache here, as it causes problems later

		for (int32 FindTrailIdx = 0; FindTrailIdx < ActiveParticles; FindTrailIdx++)
		{
			int32 CheckStartIndex = ParticleIndices[FindTrailIdx];
			DECLARE_PARTICLE_PTR(CheckParticle, ParticleData + ParticleStride * CheckStartIndex);
			FRibbonTypeDataPayload* CheckTrailData = ((FRibbonTypeDataPayload*)((uint8*)CheckParticle + TypeDataOffset));
			if (CheckTrailData->TrailIndex == TrailIdx)
			{
				if (TRAIL_EMITTER_IS_START(CheckTrailData->Flags))
				{
					StartParticle = CheckParticle;
					StartIndex = CheckStartIndex;
					StartTrailData = CheckTrailData;
					break;
				}
			}
		}

		// If we are particle sourced, and the source time is NEWER than the last source time,
		// then our source particle died... Mark the trial as dead.
		if ((TrailTypeData->bDeadTrailsOnSourceLoss == true) && (LastSourceTimes[TrailIdx] > SourceTimes[TrailIdx]))
		{
			if (StartTrailData != NULL)
			{
				StartTrailData->Flags = TRAIL_EMITTER_SET_DEADTRAIL(StartTrailData->Flags);
				SetDeadIndex(StartTrailData->TrailIndex, StartIndex);
			}
			StartParticle = NULL;
			StartIndex = 0;
			StartTrailData = NULL;
			LastSourcePosition[TrailIdx] = CurrentSourcePosition[TrailIdx];
			LastSourceRotation[TrailIdx] = CurrentSourceRotation[TrailIdx];
			LastSourceTangent[TrailIdx] = CurrentSourceTangent[TrailIdx];
			LastSourceUp[TrailIdx] = CurrentSourceUp[TrailIdx];
			LastSourceTimes[TrailIdx] = SourceTimes[TrailIdx];

			MovementSpawnCount = 0;

			// Force it to pick a new particle
			SourceIndices[TrailIdx] = -1;

			// skip to the next trail...
			continue;
		}

		bNoLivingParticles = (StartParticle == NULL);
		bool bTilingTrail = !FMath::IsNearlyZero(TrailTypeData->TilingDistance);

		FParticleEventInstancePayload* EventPayload = NULL;
		if (LODLevel->EventGenerator)
		{
			EventPayload = (FParticleEventInstancePayload*)GetModuleInstanceData(LODLevel->EventGenerator);
			if (EventPayload && !EventPayload->bSpawnEventsPresent && !EventPayload->bBurstEventsPresent)
			{
				EventPayload = NULL;
			}
		}

		float ElapsedTime = RunningTime;//SecondsSinceCreation;

		// Do we have movement based spawning?
		// If so, then interpolate the position/tangent data between 
		// CurrentSource<Position/Tangent> and LastSource<Position/Tangent>
		if (MovementSpawnCount > 0)
		{
			if (SecondsSinceCreation < TrailSpawnTimes[TrailIdx])
			{
				// Fix up the starting source tangent
				LastSourceTangent[TrailIdx] = (CurrentSourcePosition[TrailIdx] - LastSourcePosition[TrailIdx]) / ElapsedTime;
			}

			if ((TrailTypeData->bEnablePreviousTangentRecalculation == true)
				&& (TrailTypeData->bTangentRecalculationEveryFrame == false))
			{
				// Find the 2 next SPAWNED particles in the trail (not interpolated).
				// If there are 2, then the one about to be spawned will make a chain of 3 
				// giving us data to better calculate the middle ones tangent.
				// After doing so, we must also recalculate the tangents of the interpolated
				// particles in the chain.

				// The most recent spawned particle in the trail...
				FBaseParticle* NextSpawnedParticle = NULL;
				FRibbonTypeDataPayload* NextSpawnedTrailData = NULL;
				// The second most recent spawned particle in the trail...
				FBaseParticle* NextNextSpawnedParticle = NULL;
				FRibbonTypeDataPayload* NextNextSpawnedTrailData = NULL;

				FTrailsBaseTypeDataPayload* TempPayload = NULL;

				// Grab the latest two spawned particles in the trail
				GetParticleInTrail(false, StartParticle, StartTrailData, GET_Next, GET_Spawned, NextSpawnedParticle, TempPayload);
				NextSpawnedTrailData = (FRibbonTypeDataPayload*)(TempPayload);
				GetParticleInTrail(true, NextSpawnedParticle, NextSpawnedTrailData, GET_Next, GET_Spawned, NextNextSpawnedParticle, TempPayload);
				NextNextSpawnedTrailData = (FRibbonTypeDataPayload*)(TempPayload);

				if ((NextSpawnedParticle != NULL) && (NextNextSpawnedParticle != NULL))
				{
					// Calculate the new tangent from the previous and next position...
					// LastSourcePosition will be that position of the first particle that will be spawned this frame
					FVector PositionDelta = (CurrentSourcePosition[TrailIdx] - PositionOffsetThisTick - NextNextSpawnedParticle->Location);
					float TimeDelta = ElapsedTime - NextNextSpawnedTrailData->SpawnTime;

					FVector NewTangent = FVector::ZeroVector;
					if (TimeDelta > UE_SMALL_NUMBER)
					{
						NewTangent = PositionDelta / TimeDelta;
					}

					// Calculate new tangents for all the interpolated particles between NextNext and Next
					if (NextSpawnedTrailData->SpawnedTessellationPoints > 0)
					{
						FBaseParticle* CurrentParticle = NULL;
						FRibbonTypeDataPayload* CurrentTrailData = NULL;
						
						{
							int32 Prev = TRAIL_EMITTER_GET_PREV(NextNextSpawnedTrailData->Flags);
							check(Prev != TRAIL_EMITTER_NULL_PREV);
							DECLARE_PARTICLE_PTR(PrevParticle, ParticleData + ParticleStride * Prev);
							CurrentParticle = PrevParticle;
							CurrentTrailData = ((FRibbonTypeDataPayload*)((uint8*)CurrentParticle + TypeDataOffset));
						}
						
						// Fix up the next ones...
						float Diff = NextSpawnedTrailData->SpawnTime - NextNextSpawnedTrailData->SpawnTime;
						FVector CurrUp = FVector(0.0f, 0.0f, 1.0f);
						float InvCount = 1.0f / NextSpawnedTrailData->SpawnedTessellationPoints;
						// Spawn the given number of particles, interpolating between the current and last position/tangent
						//@todo. Recalculate the number of interpolated spawn particles???
						for (int32 SpawnIdx = 0; SpawnIdx < NextSpawnedTrailData->SpawnedTessellationPoints; SpawnIdx++)
						{
							float TimeStep = InvCount * (SpawnIdx + 1);
							FVector CurrPosition = FMath::CubicInterp<FVector>(
								NextNextSpawnedParticle->Location, (FVector)NextNextSpawnedTrailData->Tangent,
								NextSpawnedParticle->Location, NewTangent * Diff, 
								TimeStep);
							FVector CurrTangent = FMath::CubicInterpDerivative<FVector>(
								NextNextSpawnedParticle->Location, (FVector)NextNextSpawnedTrailData->Tangent,
								NextSpawnedParticle->Location, NewTangent * Diff,
								TimeStep);

							// Trail specific...
							CurrentParticle->OldLocation = CurrentParticle->Location;
							CurrentParticle->Location = CurrPosition;
							CurrentTrailData->Tangent = FVector3f(CurrTangent * InvCount);

							// Get the next particle in the trail (previous)
							if ((SpawnIdx + 1) < NextSpawnedTrailData->SpawnedTessellationPoints)
							{
								int32 Prev = TRAIL_EMITTER_GET_PREV(CurrentTrailData->Flags);
								check(Prev != TRAIL_EMITTER_NULL_PREV);
								DECLARE_PARTICLE_PTR(PrevParticleInTrail, ParticleData + ParticleStride * Prev);
								CurrentParticle = PrevParticleInTrail;
								CurrentTrailData = ((FRibbonTypeDataPayload*)((uint8*)CurrentParticle + TypeDataOffset));
							}
						}
					}

					// Set it for the new spawn interpolation
					LastSourceTangent[TrailIdx] = NewTangent;
				}
			}

			float LastTime = TrailSpawnTimes[TrailIdx];
			float Diff = ElapsedTime - LastTime;
			check(Diff >= 0.0f);
			FVector CurrUp = FVector(0.0f, 0.0f, 1.0f);
			float InvCount = 1.0f / MovementSpawnCount;
			float Increment = DeltaTime / MovementSpawnCount;

			FTransform SavedComponentToWorld = Component.GetComponentTransform();

			// Spawn the given number of particles, interpolating between the current and last position/tangent
			float CurrTimeStep = InvCount;
			for (int32 SpawnIdx = 0; SpawnIdx < MovementSpawnCount; SpawnIdx++, CurrTimeStep += InvCount)
			{
				float TimeStep = FMath::Clamp<float>(CurrTimeStep, 0.0f, 1.0f);
				FVector CurrPosition = FMath::CubicInterp<FVector>(
					LastSourcePosition[TrailIdx], LastSourceTangent[TrailIdx] * Diff,
					CurrentSourcePosition[TrailIdx], CurrentSourceTangent[TrailIdx] * Diff,
					TimeStep);
				FQuat CurrRotation = FQuat::Slerp(LastSourceRotation[TrailIdx], CurrentSourceRotation[TrailIdx], TimeStep);
				FVector CurrTangent = FMath::CubicInterpDerivative<FVector>(
					LastSourcePosition[TrailIdx], LastSourceTangent[TrailIdx] * Diff,
					CurrentSourcePosition[TrailIdx], CurrentSourceTangent[TrailIdx] * Diff,
					TimeStep);
				if (TrailTypeData->RenderAxis == Trails_SourceUp)
				{
					// Only interpolate the Up if using the source Up
					CurrUp = FMath::Lerp<FVector>(LastSourceUp[TrailIdx], CurrentSourceUp[TrailIdx], TimeStep);
				}
				else if (TrailTypeData->RenderAxis == Trails_WorldUp)
				{
					CurrUp = FVector(0.0f, 0.0f, 1.0f);
				}

				//@todo. Need to interpolate colors here as well!!!!

				int32 ParticleIndex = ParticleIndices[ActiveParticles];
				DECLARE_PARTICLE_PTR(Particle, ParticleData + ParticleStride * ParticleIndex);
				FRibbonTypeDataPayload* TrailData = ((FRibbonTypeDataPayload*)((uint8*)Particle + TypeDataOffset));

				// We are going from 'oldest' to 'newest' for this spawn, so reverse the time
				float StoredSpawnTime = Diff * (1.0f - TimeStep);
				float SpawnTime = DeltaTime - (SpawnIdx * Increment);
				float TrueSpawnTime = Diff * TimeStep;

				Component.SetComponentToWorld(FTransform(CurrRotation, CurrPosition));

				// Standard spawn setup
				PreSpawn(Particle, CurrPosition, FVector::ZeroVector);
				SetDeadIndex(TrailData->TrailIndex, ParticleIndex);

				for (int32 SpawnModuleIdx = 0; SpawnModuleIdx < LODLevel->SpawnModules.Num(); SpawnModuleIdx++)
				{
					UParticleModule* SpawnModule = LODLevel->SpawnModules[SpawnModuleIdx];
					if (!SpawnModule || !SpawnModule->bEnabled)
					{
						continue;
					}

					SpawnModule->Spawn({ *this, (int32)GetModuleDataOffset(SpawnModule), SpawnTime, Particle });
				}

				if (LODLevel->TypeDataModule)
				{
					//@todo. Need to track TypeData offset into payload!
					LODLevel->TypeDataModule->Spawn({ *this, TypeDataOffset, SpawnTime, Particle });
				}
				const float InterpolationPercentage = GSkipRibbonSpawnInterp ? 0.f : (1.f - float(SpawnIdx + 1) / float(MovementSpawnCount));
				PostSpawn(Particle, InterpolationPercentage, SpawnTime);

				GetParticleLifetimeAndSize(TrailIdx, Particle, bNoLivingParticles, Particle->OneOverMaxLifetime, Particle->Size.X);
				Particle->RelativeTime = SpawnTime * Particle->OneOverMaxLifetime;
				Particle->Size.Y = Particle->Size.X;
				Particle->BaseSize = Particle->Size;

				Component.SetComponentToWorld(SavedComponentToWorld);

				// Trail specific...
				// Clear the next and previous - just to be safe
				TrailData->Flags = TRAIL_EMITTER_SET_NEXT(TrailData->Flags, TRAIL_EMITTER_NULL_NEXT);
				TrailData->Flags = TRAIL_EMITTER_SET_PREV(TrailData->Flags, TRAIL_EMITTER_NULL_PREV);
				// Set the trail-specific data on this particle
				TrailData->TrailIndex = TrailIdx;
				TrailData->Tangent = FVector3f(CurrTangent * InvCount);
				TrailData->SpawnTime = ElapsedTime - StoredSpawnTime;
				TrailData->SpawnDelta = TrueSpawnTime;
				TrailData->Up = (FVector3f)CurrUp;

				TrailData->bMovementSpawned = true;

				if (SpawnIdx == (MovementSpawnCount-1))
				{
					// If this is the true spawned particle, store off the spawn interpolated count
					TrailData->bInterpolatedSpawn = false; 
					TrailData->SpawnedTessellationPoints = MovementSpawnCount;
				}
				else
				{
					TrailData->bInterpolatedSpawn = true; 
					TrailData->SpawnedTessellationPoints = 1;
				}
				TrailData->SpawnedTessellationPoints = MovementSpawnCount;

				bool bAddedParticle = false;
				// Determine which trail to attach to
				if (bNoLivingParticles)
				{
					// These are the first particles!
					// Tag it as the 'only'
					TrailData->Flags = TRAIL_EMITTER_SET_ONLY(TrailData->Flags);
					TiledUDistanceTraveled[TrailIdx] = 0.0f;
					TrailData->TiledU = 0.0f;
					bNoLivingParticles	= false;
					bAddedParticle		= true;
					SetStartIndex(TrailData->TrailIndex, ParticleIndex);
				}
				else if (StartParticle)
				{
					bAddedParticle = AddParticleHelper(TrailIdx,
						StartIndex, (FTrailsBaseTypeDataPayload*)StartTrailData, 
						ParticleIndex, (FTrailsBaseTypeDataPayload*)TrailData
						);
				}

				if (bAddedParticle)
				{
					if (bTilingTrail == true)
					{
						if (StartParticle == NULL)
						{
							TrailData->TiledU = 0.0f;
						}
						else
						{
							FVector PositionDelta = Particle->Location - StartParticle->Location;
							TiledUDistanceTraveled[TrailIdx] += PositionDelta.Size();
							TrailData->TiledU = TiledUDistanceTraveled[TrailIdx] / TrailTypeData->TilingDistance;
							//@todo. Is there going to be a problem when distance gets REALLY high?
						}
					}

					StartParticle = Particle;
					StartIndex = ParticleIndex;
					StartTrailData = TrailData;

					ActiveParticles++;
					//				check((int32)ActiveParticles < TrailTypeData->MaxParticleInTrailCount);
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
					if ((int32)ActiveParticles > LocalMaxParticleInTrailCount)
					{
						if (Component.HasWorld())
						{
							FString ErrorMessage = 
								FString::Printf(TEXT("Ribbon with too many particles: %5d vs. %5d, %s"), 
								ActiveParticles, LocalMaxParticleInTrailCount,
								Component.GetTemplate() ? *(Component.GetTemplate()->GetName()) : TEXT("No template"));
							UE_LOGF(LogParticles, Log, "%ls", *ErrorMessage);
						}
					}
#endif	//#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
					INC_DWORD_STAT(STAT_TrailParticlesSpawned);
				}
				else
				{
					check(TEXT("Failed to add particle to trail!!!!"));
				}
			}

			// Update the last position
			LastSourcePosition[TrailIdx] = CurrentSourcePosition[TrailIdx];
			LastSourceRotation[TrailIdx] = CurrentSourceRotation[TrailIdx];
			LastSourceTangent[TrailIdx] = CurrentSourceTangent[TrailIdx];
			LastSourceUp[TrailIdx] = CurrentSourceUp[TrailIdx];
			TrailSpawnTimes[TrailIdx] = ElapsedTime;
			LastSourceTimes[TrailIdx] = SourceTimes[TrailIdx];
			if ((SourceModule != NULL) && (SourceModule->SourceMethod == PET2SRCM_Particle))
			{
				if (SourceTimes[TrailIdx] > 1.0f)
				{
					StartTrailData->Flags = TRAIL_EMITTER_SET_DEADTRAIL(StartTrailData->Flags);
					SourceIndices[TrailIdx] = -1;
					SetDeadIndex(StartTrailData->TrailIndex, StartIndex);
				}
			}
		}
	}

	INC_DWORD_STAT_BY(STAT_TrailParticles, ActiveParticles);

	return bProcessSpawnRate;
}

/**
 *	Spawn ribbon particles from SpawnRate and Burst settings.
 *
 *	@param	DeltaimTime			The current time slice
 *	
 *	@return	float				The spawnfraction left over from this time slice
 */
float FParticleRibbonEmitterInstance::Spawn_RateAndBurst(float DeltaTime)
{
	return SpawnFraction;
}

void FParticleRibbonEmitterInstance::SetupTrailModules()
{
	// Trails are a special case... 
	// We don't want standard Spawn/Update calls occuring on Trail-type modules.
	UParticleLODLevel* LODLevel = SpriteTemplate->GetLODLevel(0);
	check(LODLevel);
	for (int32 ModuleIdx = 0; ModuleIdx < LODLevel->Modules.Num(); ModuleIdx++)
	{
		bool bRemoveIt = false;
		UParticleModule* CheckModule = LODLevel->Modules[ModuleIdx];
		UParticleModuleSpawnPerUnit* CheckSPUModule = Cast<UParticleModuleSpawnPerUnit>(CheckModule);
		UParticleModuleTrailSource*	CheckSourceModule = Cast<UParticleModuleTrailSource>(CheckModule);

		if (CheckSPUModule != NULL)
		{
			SpawnPerUnitModule = CheckSPUModule;
			bRemoveIt = true;
		}
		else if (CheckSourceModule != NULL)
		{
			SourceModule = CheckSourceModule;
			uint32* Offset = SpriteTemplate->ModuleOffsetMap.Find(CheckSourceModule);
			if (Offset != NULL)
			{
				TrailModule_Source_Offset = *Offset;
			}
			bRemoveIt = true;
		}

		if (bRemoveIt == true)
		{
			// Remove it from any lists...
			for (int32 UpdateIdx = LODLevel->UpdateModules.Num() - 1; UpdateIdx >= 0; UpdateIdx--)
			{
				if (LODLevel->UpdateModules[UpdateIdx] == CheckModule)
				{
					LODLevel->UpdateModules.RemoveAt(UpdateIdx);
				}
			}

			for (int32 SpawnIdx = LODLevel->SpawnModules.Num() - 1; SpawnIdx >= 0; SpawnIdx--)
			{
				if (LODLevel->SpawnModules[SpawnIdx] == CheckModule)
				{
					LODLevel->SpawnModules.RemoveAt(SpawnIdx);
				}
			}

			for (int32 SpawningIdx = LODLevel->SpawningModules.Num() - 1; SpawningIdx >= 0; SpawningIdx--)
			{
				if (LODLevel->SpawningModules[SpawningIdx] == CheckModule)
				{
					LODLevel->SpawningModules.RemoveAt(SpawningIdx);
				}
			}
		}
	}
}

void FParticleRibbonEmitterInstance::ResolveSource()
{
	if (SourceModule && SourceModule->SourceName != NAME_None)
	{
		switch (SourceModule->SourceMethod)
		{
		case PET2SRCM_Actor:
			if (SourceActor == NULL)
			{
				TArrayView<const FParticleSysParam> AsyncInstanceParameters = Component.GetAsyncInstanceParameters();
				FParticleSysParam Param;
				for (int32 ParamIdx = 0; ParamIdx < AsyncInstanceParameters.Num(); ParamIdx++)
				{
					Param = AsyncInstanceParameters[ParamIdx];
					if (Param.Name == SourceModule->SourceName)
					{
						SourceActor = Param.Actor;
						break;
					}
				}

				if (SourceModule->SourceOffsetCount > 0)
				{
					for (int32 ParamIdx = 0; ParamIdx < AsyncInstanceParameters.Num(); ParamIdx++)
					{
						Param = AsyncInstanceParameters[ParamIdx];
						FString ParamName = Param.Name.ToString();
						const TCHAR* TrailSourceOffset = FCString::Strstr(*ParamName, TEXT("TrailSourceOffset"));
						if (TrailSourceOffset)
						{
							// Parse off the digit
							int32	Index	= FCString::Atoi(TrailSourceOffset);
							if (Index >= 0)
							{
								if (Param.ParamType	== PSPT_Vector)
								{
									SourceOffsets.InsertUninitialized(Index);
									SourceOffsets[Index] = Param.Vector;
								}
								else if (Param.ParamType == PSPT_Scalar)
								{
									SourceOffsets.InsertZeroed(Index);
									SourceOffsets[Index] = FVector(Param.Scalar, 0.0f, 0.0f);
								}
							}
						}
					}
				}
			}
			break;
		case PET2SRCM_Particle:
			if (SourceEmitter == NULL)
			{
				TArrayView<FParticleEmitterInstance*> EmitterInstances = Component.GetEmitterInstances();
				for (int32 EmitterIdx = 0; EmitterIdx < EmitterInstances.Num(); EmitterIdx++)
				{
					FParticleEmitterInstance* EmitInst = EmitterInstances[EmitterIdx];
					if (EmitInst && (EmitInst->SpriteTemplate->EmitterName == SourceModule->SourceName))
					{
						SourceEmitter = EmitInst;
						break;
					}
				}
			}
			break;
		}
	}
}

void FParticleRibbonEmitterInstance::UpdateSourceData(float DeltaTime, bool bFirstTime)
{
	FVector	Position;
	FQuat Rotation;
	FVector	Tangent;
	FVector	Up;
	float TangentStrength;
	// For each possible trail in this emitter, update it's source information
	float ElapsedTime = RunningTime;//SecondsSinceCreation;
	bool bCanBeValidParticleSource = ((SourceModule != NULL) && (SourceModule->SourceMethod == PET2SRCM_Particle));
	for (int32 TrailIdx = 0; TrailIdx < MaxTrailCount; TrailIdx++)
	{
		bool bNewSource = (SourceIndices[TrailIdx] == -1);
		if (ResolveSourcePoint(TrailIdx, Position, Rotation, Up, Tangent, TangentStrength) == true)
		{
			if (SourceIndices[TrailIdx] == -1 && bCanBeValidParticleSource)
			{
				//No valid particle for source so set all last and prev data to the same defaults. Stops issues with SpawnPerUnit trails having a segment back to the default location.
				LastSourcePosition[TrailIdx] = Position;
				LastSourceTangent[TrailIdx] = Tangent;
				LastSourceTangentStrength[TrailIdx] = TangentStrength;
				LastSourceRotation[TrailIdx] = Rotation;
				LastSourceUp[TrailIdx] = Up;

				CurrentSourcePosition[TrailIdx] = Position;
				CurrentSourceTangent[TrailIdx] = Tangent;
				CurrentSourceTangentStrength[TrailIdx] = TangentStrength;
				CurrentSourceRotation[TrailIdx] = Rotation;
				CurrentSourceUp[TrailIdx] = Up;

				TrailSpawnTimes[TrailIdx] = 0.0f;				
			}
			else
			{
				if ((bFirstTime == true) || 
					((bNewSource == true) && bCanBeValidParticleSource))
				{
					LastSourcePosition[TrailIdx] = Position;
					LastSourceTangent[TrailIdx] = FVector::ZeroVector;//Component->LocalToWorld.TransformVector(FVector(1,0,0));
					LastSourceTangentStrength[TrailIdx] = TangentStrength;
					LastSourceUp[TrailIdx] = Up;
					TrailSpawnTimes[TrailIdx] = RunningTime;
				}
				CurrentSourcePosition[TrailIdx] = Position;
				CurrentSourceRotation[TrailIdx] = Rotation;
				float ElapsedTimeSinceSpanwed = ElapsedTime - TrailSpawnTimes[TrailIdx];
				if ( ElapsedTimeSinceSpanwed != 0.f )
				{
					CurrentSourceTangent[TrailIdx] = (CurrentSourcePosition[TrailIdx] - LastSourcePosition[TrailIdx]) / ElapsedTimeSinceSpanwed;
				}
				else
				{
					CurrentSourceTangent[TrailIdx] = FVector(1.f, 0.f, 0.f);
				}
				CurrentSourceTangentStrength[TrailIdx] = TangentStrength;
				CurrentSourceUp[TrailIdx] = Up;
				if (bFirstTime == true)
				{
					LastSourceRotation[TrailIdx] = CurrentSourceRotation[TrailIdx];
				}
			}
		}
	}
}

/**
 *	Resolve the source point for the given trail index.
 *
 *	@param	InTrailIdx			The index of the trail to resolve
 *	@param	OutPosition			The position of the source
 *	@param	OutRotation			The rotation of the source
 *	@param	OutUp				The 'up' of the source (if required)
 *	@param	OutTangent			The tangent of the source
 *	@param	OutTangentStrength	The strength of the tangent of the source
 *
 *	@return	bool				true if successful, false if not
 */
bool FParticleRibbonEmitterInstance::ResolveSourcePoint(int32 InTrailIdx, 
	FVector& OutPosition, FQuat& OutRotation, FVector& OutUp, FVector& OutTangent, float& OutTangentStrength)
{
	const FTransform& AsyncComponentToWorld = Component.GetAsyncComponentToWorld();
	bool bSourceWasSet = false;
	// Resolve the source point...
	if (SourceModule)
	{
		switch (SourceModule->SourceMethod)
		{
		case PET2SRCM_Particle:
			{
				if (SourceEmitter == NULL)
				{
					// Is this the first time?
					ResolveSource();
				}

				if (SourceEmitter && SourceEmitter->ParticleIndices && SourceEmitter->ActiveParticles > 0)
				{
					if (SourceIndices[InTrailIdx] != -1)
					{
						FBaseParticle* SourceParticle = SourceEmitter->GetParticleDirect(SourceIndices[InTrailIdx]);
						if (SourceParticle == NULL || SourceParticle->RelativeTime>1.0f)
						{
							// If the previous particle is not found, force the trail to pick a new one
							SourceIndices[InTrailIdx] = -1;
						}
					}

					if (SourceIndices[InTrailIdx] == -1 && SourceEmitter->ActiveParticles > 0)
					{
						int32 Index = 0;
						switch (SourceModule->SelectionMethod)
						{
							case EPSSM_Random:
							{
							FRandomStream& RandomStream = SourceModule->GetRandomStream({*this});
								Index = RandomStream.RandHelper(SourceEmitter->ActiveParticles);
							}
								break;
							case EPSSM_Sequential:
							{
								bool bInUse = false;
								LastSelectedParticleIndex++;
								if (LastSelectedParticleIndex >= SourceEmitter->ActiveParticles)
								{
									LastSelectedParticleIndex = 0;
								}

								// need to check if the next source index is in use, and go through until we find one that isn't
								do 
								{
									bInUse = false;
									for (int32 TrailCheckIdx = 0; TrailCheckIdx < MaxTrailCount; TrailCheckIdx++)
									{
										if (TrailCheckIdx != InTrailIdx && SourceIndices[TrailCheckIdx] == SourceEmitter->ParticleIndices[LastSelectedParticleIndex])
										{
											bInUse = true;
										}
									}
									if (bInUse == true)
									{
										LastSelectedParticleIndex++;
									}								
								} while (bInUse && LastSelectedParticleIndex < SourceEmitter->ActiveParticles);

								if (LastSelectedParticleIndex >= SourceEmitter->ActiveParticles)
								{
									LastSelectedParticleIndex = -1;
								}

								Index = LastSelectedParticleIndex;
							}
							break;
						}

						SourceIndices[InTrailIdx] = Index != -1 ? SourceEmitter->ParticleIndices[Index] : -1;
					}

					bool bEncounteredNaNError = false;

					// Grab the particle

					const int32 SourceEmitterParticleIndex = SourceIndices[InTrailIdx];
					FBaseParticle* SourceParticle = ((SourceEmitterParticleIndex >= 0)) ? SourceEmitter->GetParticleDirect(SourceEmitterParticleIndex) : nullptr;
					if (SourceParticle != nullptr)
					{
						const FVector WorldOrigin = SourceEmitter->SimulationToWorld.GetOrigin();
						IParticleEmitterInstanceOwner& Comp = SourceEmitter->Component;
						if (!ensureMsgf(!SourceParticle->Location.ContainsNaN(), TEXT("NaN in SourceParticle Location. Template: %s, Component: %s"), *GetNameSafe(Comp.GetTemplate()), *Comp.GetPathName()) ||
							!ensureMsgf(!SourceParticle->OldLocation.ContainsNaN(), TEXT("NaN in SourceParticle OldLocation. Template: %s, Component: %s"), *GetNameSafe(Comp.GetTemplate()), *Comp.GetPathName()) ||
							!ensureMsgf(!WorldOrigin.ContainsNaN(), TEXT("NaN in WorldOrigin. Template: %s, Component: %s"), *GetNameSafe(Comp.GetTemplate()), *Comp.GetPathName())
							)
						{
							UE_LOGF(LogParticles, Warning, "TrailIdx: %d", InTrailIdx);
							UE_LOGF(LogParticles, Warning, "SourceEmitterParticleIndex: %d", SourceEmitterParticleIndex);
							UE_LOGF(LogParticles, Warning, "ActiveParticles: %d", SourceEmitter->ActiveParticles);

							UE_LOGF(LogParticles, Warning, "==============ParticleIndices=================");
							for (int32 i = 0; i < SourceEmitter->ActiveParticles; ++i)
							{
								UE_LOGF(LogParticles, Warning, "%d: %d", i, SourceEmitter->ParticleIndices[i]);
							}

							UE_LOGF(LogParticles, Warning, "==============ParticleData=================");
							for (int32 i = 0; i < SourceEmitter->ActiveParticles; ++i)
							{
								UE_LOGF(LogParticles, Warning, "-- Particle %d --", i);
								if (FBaseParticle* DumpParticle = SourceEmitter->GetParticleDirect(i))
								{
									UE_LOGF(LogParticles, Warning, "Location:{%6.4f, %6.4f, %6.4f}", DumpParticle->Location.X, DumpParticle->Location.Y, DumpParticle->Location.Z);
									UE_LOGF(LogParticles, Warning, "OldLocation:{%6.4f, %6.4f, %6.4f}", DumpParticle->OldLocation.X, DumpParticle->OldLocation.Y, DumpParticle->OldLocation.Z);
									UE_LOGF(LogParticles, Warning, "BaseVelocity:{%6.4f, %6.4f, %6.4f}", DumpParticle->BaseVelocity.X, DumpParticle->BaseVelocity.Y, DumpParticle->BaseVelocity.Z);
									UE_LOGF(LogParticles, Warning, "Velocity:{%6.4f, %6.4f, %6.4f}", DumpParticle->Velocity.X, DumpParticle->Velocity.Y, DumpParticle->Velocity.Z);
									UE_LOGF(LogParticles, Warning, "BaseSize:{%6.4f, %6.4f, %6.4f}", DumpParticle->BaseSize.X, DumpParticle->BaseSize.Y, DumpParticle->BaseSize.Z);
									UE_LOGF(LogParticles, Warning, "Size:{%6.4f, %6.4f, %6.4f}", DumpParticle->Size.X, DumpParticle->Size.Y, DumpParticle->Size.Z);
									UE_LOGF(LogParticles, Warning, "RelativeTime: %6.4f", DumpParticle->RelativeTime);
									UE_LOGF(LogParticles, Warning, "OneOverMaxLifetime: %6.4f", DumpParticle->OneOverMaxLifetime);
									UE_LOGF(LogParticles, Warning, "Rotation: %6.4f", DumpParticle->Rotation);
									UE_LOGF(LogParticles, Warning, "BaseRotationRate: %6.4f", DumpParticle->BaseRotationRate);
									UE_LOGF(LogParticles, Warning, "RotationRate: %6.4f", DumpParticle->RotationRate);
									UE_LOGF(LogParticles, Warning, "Flags: %d", DumpParticle->Flags);
									UE_LOGF(LogParticles, Warning, "Color:{%6.4f, %6.4f, %6.4f, %6.4f}", DumpParticle->Color.R, DumpParticle->Color.G, DumpParticle->Color.B, DumpParticle->Color.A);
									UE_LOGF(LogParticles, Warning, "BaseColor:{%6.4f, %6.4f, %6.4f, %6.4f}", DumpParticle->BaseColor.R, DumpParticle->BaseColor.G, DumpParticle->BaseColor.B, DumpParticle->BaseColor.A)
								}
								else
								{
									UE_LOGF(LogParticles, Warning, "Dump Particle was NULL");
								}
							}

							// Contains NaN!
							bEncounteredNaNError = true;
						}
						else
						{
							OutPosition = SourceParticle->Location + WorldOrigin;
							OutTangent = SourceParticle->Location - SourceParticle->OldLocation;
							SourceTimes[InTrailIdx] = SourceParticle->RelativeTime;
						}
					}
					else
					{
						// Fall back to the emitter location??
						OutPosition = SourceEmitter->Component.GetComponentTransform().GetLocation();
						OutTangent = Component.GetPartSysVelocity();
						//@todo. How to handle this... can potentially cause a jump from the emitter to the
						// particle...
						SourceTimes[InTrailIdx] = 0.0f;
						SourceIndices[InTrailIdx] = -1;//No valid particle source;
					}
					OutTangentStrength = OutTangent.SizeSquared();
					//@todo. Allow particle rotation to define up??
					OutUp = SourceEmitter->Component.GetComponentTransform().GetScaledAxis(EAxis::Z);

					//@todo. Where to get rotation from????
					OutRotation = FQuat(0,0,0,1);

					//@todo. Support source offset

					bSourceWasSet = !bEncounteredNaNError;
				}
			}
			break;
		case PET2SRCM_Actor:
			if (SourceModule->SourceName != NAME_None)
			{
				if (SourceActor == NULL)
				{
					ResolveSource();
				}

				if (SourceActor)
				{
					FTransform ActorToWorld = SourceActor->ActorToWorld();
					OutPosition = ActorToWorld.GetLocation();
					FRotator TempRotator = ActorToWorld.Rotator();
					OutRotation = FQuat(TempRotator);
					OutTangent = SourceActor->GetVelocity();
					OutTangentStrength = OutTangent.SizeSquared();

					OutUp = ActorToWorld.TransformVector(FVector(0.f, 0.f, 1.f));

					bSourceWasSet = true;
				}
			}
			break;
		}
	}

	if (bSourceWasSet == false)
	{
		OutPosition = Component.GetComponentTransform().GetLocation();
		if (SourceModule && (SourceModule->SourceOffsetCount > 0))
		{
			FVector SourceOffsetValue;
			if (SourceModule->ResolveSourceOffset(InTrailIdx, this, SourceOffsetValue) == true)
			{
				if (CurrentLODLevel && (CurrentLODLevel->RequiredModule->bUseLocalSpace == false))
				{
					// Transform it
					SourceOffsetValue = Component.GetComponentTransform().TransformVector(SourceOffsetValue);
				}
				OutPosition += SourceOffsetValue;
			}
		}
		OutRotation = Component.GetComponentTransform().GetRotation();
		OutTangent = Component.GetPartSysVelocity();
		OutTangentStrength = OutTangent.SizeSquared();
		OutUp = Component.GetComponentTransform().GetScaledAxis(EAxis::Z);

		bSourceWasSet = true;
	}

	return bSourceWasSet;
}

/** Determine the number of vertices and triangles in each trail */
void FParticleRibbonEmitterInstance::DetermineVertexAndTriangleCount()
{
	uint32 NewSize = 0;
	int32 Sheets = 1;//TrailTypeData->Sheets ? TrailTypeData->Sheets : 1;
	int32 TheTrailCount = 0;
	int32 IndexCount = 0;

	VertexCount		= 0;
	TriangleCount	= 0;
	HeadOnlyParticles = 0;

	int32 CheckParticleCount = 0;

	int32 TempVertexCount;
	bool bApplyDistanceTessellation = !FMath::IsNearlyZero(TrailTypeData->DistanceTessellationStepSize);
	float DistTessStep = TrailTypeData->DistanceTessellationStepSize;
	const float ScaleStepFactor = 0.5f;
	bool bScaleTessellation = TrailTypeData->bEnableTangentDiffInterpScale;

	float DistDiff = 0.0f;
	float CheckTangent = 0.0f;
	bool bCheckTangentValue = !FMath::IsNearlyZero(TrailTypeData->TangentTessellationScalar) || bScaleTessellation;
	// 
	for (int32 ii = 0; ii < ActiveParticles; ii++)
	{
		int32 LocalIndexCount = 0;
		int32	ParticleCount = 0;
		int32 LocalVertexCount = 0;
		int32 LocalTriCount = 0;

		bool bProcessParticle = false;

		DECLARE_PARTICLE_PTR(Particle, ParticleData + ParticleStride * ParticleIndices[ii]);
		FBaseParticle* CurrParticle = Particle;
		FRibbonTypeDataPayload*	CurrTrailData = ((FRibbonTypeDataPayload*)((uint8*)Particle + TypeDataOffset));
		if (TRAIL_EMITTER_IS_HEADONLY(CurrTrailData->Flags))
		{
			// If there is only a single particle in the trail, then we only want to render it
			// if we are connecting to the source...
			//@todo. Support clip source segment
			CurrTrailData->RenderingInterpCount = 0;
			CurrTrailData->TriangleCount = 0;
			++HeadOnlyParticles;
		}
		else if (TRAIL_EMITTER_IS_END(CurrTrailData->Flags))
		{
			// Walk from the end of the trail to the front
			FBaseParticle* PrevParticle = NULL;
			FRibbonTypeDataPayload*	PrevTrailData = NULL;
			int32	Prev = TRAIL_EMITTER_GET_PREV(CurrTrailData->Flags);
			if (Prev != TRAIL_EMITTER_NULL_PREV)
			{
				DECLARE_PARTICLE_PTR(InnerParticle, ParticleData + ParticleStride * Prev);
				PrevParticle = InnerParticle;
				PrevTrailData = ((FRibbonTypeDataPayload*)((uint8*)InnerParticle + TypeDataOffset));

				bool bDone = false;
				// The end of the trail, so there MUST be another particle
				while (!bDone)
				{
					ParticleCount++;
					// Determine the number of rendered interpolated points between these two particles
					float CheckDistance = (CurrParticle->Location - PrevParticle->Location).Size();
					FVector SrcTangent = (FVector)CurrTrailData->Tangent;
					SrcTangent.Normalize();
					FVector PrevTangent = (FVector)PrevTrailData->Tangent;
					PrevTangent.Normalize();
					if (bCheckTangentValue == true)
					{
						CheckTangent = (SrcTangent | PrevTangent);
						// Map the tangent difference to [0..1] for [0..180]
						//  1.0 = parallel    --> -1 = 0
						//  0.0 = orthogonal  --> -1 = -1 --> * -0.5 = 0.5
						// -1.0 = oppositedir --> -1 = -2 --> * -0.5 = 1.0
						CheckTangent = (CheckTangent - 1.0f) * -0.5f;
					}

					if (bApplyDistanceTessellation == true)
					{
						DistDiff = CheckDistance / DistTessStep;
						if (bScaleTessellation && (CheckTangent < ScaleStepFactor))
						{
							// Scale the tessellation step size so that 
							// parallel .. orthogonal maps to [0..1] (and > 90 also == 1.0)
							DistDiff *= (2.0f * FMath::Clamp<float>(CheckTangent, 0.0f, 0.5f));
						}
					}

					//@todo. Need to adjust the tangent diff count when the distance is REALLY small...
					float TangDiff = CheckTangent * TrailTypeData->TangentTessellationScalar;
					int32 InterpCount = FMath::Min(FMath::TruncToInt(DistDiff), FXConsoleVariables::MaxDistanceTessellation) + FMath::Min(FMath::TruncToInt(TangDiff), FXConsoleVariables::MaxTangentTessellation);

					// There always is at least 1 point (the source particle itself)
					InterpCount = (InterpCount > 0) ? InterpCount : 1;

					// Store off the rendering interp count for this particle
					CurrTrailData->RenderingInterpCount = InterpCount;
					if (CheckTangent <= 0.5f)
					{
						CurrTrailData->PinchScaleFactor = 1.0f;
					}
					else
					{
						CurrTrailData->PinchScaleFactor = 1.0f - (CheckTangent * 0.5f);
					}

					// Tally up the vertex and index counts for this segment...
					TempVertexCount = 2 * InterpCount * Sheets;
					VertexCount += TempVertexCount;
					LocalVertexCount += TempVertexCount;
					LocalIndexCount += TempVertexCount;

					// Move to the previous particle in the chain
					CurrParticle = PrevParticle;
					CurrTrailData = PrevTrailData;
					Prev = TRAIL_EMITTER_GET_PREV(CurrTrailData->Flags);
					if (Prev != TRAIL_EMITTER_NULL_PREV)
					{
						DECLARE_PARTICLE_PTR(PrevParticlePtr, ParticleData + ParticleStride * Prev);
						PrevParticle = PrevParticlePtr;
						PrevTrailData = ((FRibbonTypeDataPayload*)((uint8*)PrevParticle + TypeDataOffset));
					}
					else
					{
						// The START particle will have a previous index of NULL, so we're done
						bDone = true;
					}
				}

				bProcessParticle = true;
			}
			else
			{
				// This means there is only a single particle in the trail - the end...
// 				check(!TEXT("FAIL"));
				bProcessParticle = false;
			}
		}

		if (bProcessParticle == true)
		{
			// The last step is the last interpolated step to the Curr (which should be the start)
			ParticleCount++;
			TempVertexCount = 2 * Sheets;
			VertexCount += TempVertexCount;
			LocalVertexCount += TempVertexCount;
			LocalIndexCount += TempVertexCount;

			// If we are running up to the current source, take that into account as well.
			//@todo. Support clip source segment

			// Handle degenerates - 4 tris per stitch
			LocalIndexCount	+= ((Sheets - 1) * 4);

			// @todo: We're going and modifying the original ParticleData here!  This is kind of sketchy
			//    since it's not supposed to be changed at this phase
			check(TRAIL_EMITTER_IS_HEAD(CurrTrailData->Flags));
			CurrTrailData->TriangleCount = LocalIndexCount - 2;

			// The last particle in the chain will always have 1 here!
			CurrTrailData->RenderingInterpCount = 1;

			// Increment the total index count
			IndexCount += LocalIndexCount;
			TheTrailCount++;
		}
	}

	TrailCount = TheTrailCount;
	if (TheTrailCount > 0)
	{
		IndexCount += 4 * (TheTrailCount - 1);	// 4 extra indices per Trail (degenerates)
		TriangleCount = IndexCount - (2 * TheTrailCount);
	}
	else
	{
		IndexCount = 0;
		TriangleCount = 0;
	}
}

/**
 *	Checks some common values for GetDynamicData validity
 *
 *	@return	bool		true if GetDynamicData should continue, false if it should return NULL
 */
bool FParticleRibbonEmitterInstance::IsDynamicDataRequired(UParticleLODLevel* InCurrentLODLevel)
{
	if (FParticleEmitterInstance::IsDynamicDataRequired(InCurrentLODLevel) == true)
	{
		if (/*(TrailTypeData->bClipSourceSegement == true) &&*/ (ActiveParticles < 2))
		{
			return false;
		}
	}
	return true;
}

/**
 *	Retrieves the dynamic data for the emitter
 */
FDynamicEmitterDataBase* FParticleRibbonEmitterInstance::GetDynamicData(bool bSelected, ERHIFeatureLevel::Type InFeatureLevel)
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_ParticleRibbonEmitterInstance_GetDynamicData);

	UParticleLODLevel* LODLevel = SpriteTemplate->GetLODLevel(0);
	if (IsDynamicDataRequired(LODLevel) == false || !bEnabled)
	{
		return NULL;
	}

	// Allocate the dynamic data
	FDynamicRibbonEmitterData* NewEmitterData = new FDynamicRibbonEmitterData(LODLevel->RequiredModule);
	{
		INC_DWORD_STAT(STAT_DynamicEmitterCount);
		INC_DWORD_STAT(STAT_DynamicRibbonCount);
		INC_DWORD_STAT_BY(STAT_DynamicEmitterMem, sizeof(FDynamicRibbonEmitterData));
	}

	NewEmitterData->bClipSourceSegement = TrailTypeData->bClipSourceSegement;
	NewEmitterData->bRenderGeometry = TrailTypeData->bRenderGeometry;
	NewEmitterData->bRenderParticles = TrailTypeData->bRenderSpawnPoints;
	NewEmitterData->bRenderTangents = TrailTypeData->bRenderTangents;
	NewEmitterData->bRenderTessellation = TrailTypeData->bRenderTessellation;
	NewEmitterData->DistanceTessellationStepSize = TrailTypeData->DistanceTessellationStepSize;
	NewEmitterData->TangentTessellationScalar = TrailTypeData->TangentTessellationScalar;
	NewEmitterData->RenderAxisOption = TrailTypeData->RenderAxis;
	NewEmitterData->TextureTileDistance = TrailTypeData->TilingDistance;
	if (NewEmitterData->TextureTileDistance > 0.0f)
	{
		NewEmitterData->bTextureTileDistance = true;
	}
	else
	{
		NewEmitterData->bTextureTileDistance = false;
	}

	// Now fill in the source data
	if (!FillReplayData(NewEmitterData->Source))
	{
		delete NewEmitterData;
		return NULL;
	}

	// Setup dynamic render data.  Only call this AFTER filling in source data for the emitter.
	NewEmitterData->Init(bSelected);

	return NewEmitterData;
}

/**
 *	Retrieves replay data for the emitter
 *
 *	@return	The replay data, or NULL on failure
 */
FDynamicEmitterReplayDataBase* FParticleRibbonEmitterInstance::GetReplayData()
{
	if (ActiveParticles <= 0 || !bEnabled)
	{
		return NULL;
	}

	FDynamicEmitterReplayDataBase* NewEmitterReplayData = new FDynamicRibbonEmitterReplayData();
	check(NewEmitterReplayData != NULL);

	if (!FillReplayData(*NewEmitterReplayData))
	{
		delete NewEmitterReplayData;
		return NULL;
	}

	return NewEmitterReplayData;
}

/**
 *	Retrieve the allocated size of this instance.
 *
 *	@param	OutNum			The size of this instance
 *	@param	OutMax			The maximum size of this instance
 */
void FParticleRibbonEmitterInstance::GetAllocatedSize(int32& OutNum, int32& OutMax)
{
	int32 Size = sizeof(FParticleRibbonEmitterInstance);
	int32 ActiveParticleDataSize = (ParticleData != NULL) ? (ActiveParticles * ParticleStride) : 0;
	int32 MaxActiveParticleDataSize = (ParticleData != NULL) ? (MaxActiveParticles * ParticleStride) : 0;
	int32 ActiveParticleIndexSize = (ParticleIndices != NULL) ? (ActiveParticles * sizeof(uint16)) : 0;
	int32 MaxActiveParticleIndexSize = (ParticleIndices != NULL) ? (MaxActiveParticles * sizeof(uint16)) : 0;

	OutNum = ActiveParticleDataSize + ActiveParticleIndexSize + Size;
	OutMax = MaxActiveParticleDataSize + MaxActiveParticleIndexSize + Size;
}

/**
 * Returns the size of the object/ resource for display to artists/ LDs in the Editor.
 *
 * @param	Mode	Specifies which resource size should be displayed. ( see EResourceSizeMode::Type )
 * @return  Size of resource as to be displayed to artists/ LDs in the Editor.
 */
void FParticleRibbonEmitterInstance::GetResourceSizeEx(FResourceSizeEx& CumulativeResourceSize)
{
	if (CumulativeResourceSize.GetResourceSizeMode() == EResourceSizeMode::EstimatedTotal || (Component.GetSceneProxy()))
	{
		int32 MaxActiveParticleDataSize = (ParticleData != NULL) ? (MaxActiveParticles * ParticleStride) : 0;
		int32 MaxActiveParticleIndexSize = (ParticleIndices != NULL) ? (MaxActiveParticles * sizeof(uint16)) : 0;
		// Take dynamic data into account as well
		CumulativeResourceSize.AddUnknownMemoryBytes(sizeof(FParticleRibbonEmitterInstance));
		CumulativeResourceSize.AddUnknownMemoryBytes(MaxActiveParticleDataSize);								// Copy of the particle data on the render thread
		CumulativeResourceSize.AddUnknownMemoryBytes(MaxActiveParticleIndexSize);								// Copy of the particle indices on the render thread
		if (DynamicParameterDataOffset == 0)
		{
			CumulativeResourceSize.AddUnknownMemoryBytes(MaxActiveParticles * sizeof(FParticleBeamTrailVertex));	// The vertex data array
		}
		else
		{
			CumulativeResourceSize.AddUnknownMemoryBytes(MaxActiveParticles * sizeof(FParticleBeamTrailVertexDynamicParameter));
		}
	}
}

/**
 * Captures dynamic replay data for this particle system.
 *
 * @param	OutData		[Out] Data will be copied here
 *
 * @return Returns true if successful
 */
bool FParticleRibbonEmitterInstance::FillReplayData(FDynamicEmitterReplayDataBase& OutData)
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_ParticleRibbonEmitterInstance_FillReplayData);

	if (ActiveParticles <= 0 || !bEnabled)
	{
		return false;
	}

	// If the template is disabled, don't return data.
	UParticleLODLevel* LODLevel = SpriteTemplate->GetLODLevel(0);
	if ((LODLevel == NULL) || (LODLevel->bEnabled == false))
	{
		return false;
	}

	// This function can modify the ParticleData (changes TriangleCount of trail payloads), so we
	// we need to call it before calling the parent implementation of FillReplayData, since that
	// will memcpy the particle data to the render thread's buffer.
	DetermineVertexAndTriangleCount();

	const int32 IndexCount = TriangleCount + 2;
	if (IndexCount > MAX_TRAIL_INDICES)
	{
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
		UE_LOGF(LogParticles, Warning, "RIBBON   : FillReplayData failed.");
		UE_LOGF(LogParticles, Warning, "\tIndexCount (%d) exceeds allowed value (%d).", IndexCount, MAX_TRAIL_INDICES);
		UE_LOGF(LogParticles, Warning, "\tActiveParticleCount = %d.", ActiveParticles);
		UE_LOGF(LogParticles, Warning, "\tTriangleCount = %d.", TriangleCount);
		UE_LOGF(LogParticles, Warning, "\tTrailCount = %d.", TrailCount);
		UE_LOGF(LogParticles, Warning, "\t%ls", Component.GetTemplate() ? 
			*(Component.GetTemplate()->GetPathName()) : *(Component.GetName()));
#endif
		return false;
	}

	// Call parent implementation first to fill in common particle source data
	if (!FParticleEmitterInstance::FillReplayData(OutData))
	{
		return false;
	}

	if (TriangleCount <= 0)
	{
		if ((ActiveParticles > 0) && (ActiveParticles != HeadOnlyParticles))
		{
			if (!TrailTypeData->bClipSourceSegement)
			{
// 				UE_LOGF(LogParticles, Warning, "TRAIL: GetDynamicData -- TriangleCount == 0 (APC = %4d) for PSys %ls",
// 					ActiveParticles, 
// 					Component ? (Component->Template ? *Component->Template->GetName() : 
// 					TEXT("No Template")) : TEXT("No Component"));
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
				if (Component.HasWorld())
				{
					FString ErrorMessage = 
						FString::Printf(TEXT("RIBBON: GetDynamicData -- TriangleCount == %d (APC = %4d) for PSys %s"),
							TriangleCount, ActiveParticles, 
							Component.GetTemplate() ? *Component.GetTemplate()->GetName() : TEXT("No Template"));
					UE_LOGF(LogParticles, Log, "%ls", *ErrorMessage);
				}
#endif	//#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
			}
		}
		return false;
	}

	OutData.eEmitterType = DET_Ribbon;
	FDynamicRibbonEmitterReplayData* NewReplayData = static_cast<FDynamicRibbonEmitterReplayData*>( &OutData );

	NewReplayData->MaterialInterface = GetCurrentMaterial();
	// We never want local space for trails
	NewReplayData->bUseLocalSpace = false;
	// Never use axis lock for trails
	NewReplayData->bLockAxis = false;
	
	NewReplayData->MaxActiveParticleCount = MaxActiveParticles;
	NewReplayData->MaxTessellationBetweenParticles = TrailTypeData->MaxTessellationBetweenParticles ? TrailTypeData->MaxTessellationBetweenParticles : 1;
	NewReplayData->Sheets = TrailTypeData->SheetsPerTrail ? TrailTypeData->SheetsPerTrail : 1;
	NewReplayData->Sheets = FMath::Max(NewReplayData->Sheets, 1);

	NewReplayData->VertexCount = VertexCount;
	NewReplayData->IndexCount = TriangleCount + 2;
	NewReplayData->PrimitiveCount = TriangleCount;
	NewReplayData->TrailCount = TrailCount;

	//@todo.SAS. Check for requiring uint32 sized indices?
	NewReplayData->IndexStride = sizeof(uint16);

	NewReplayData->TrailDataOffset = TypeDataOffset;

	return true;
}

void FParticleRibbonEmitterInstance::ApplyWorldOffset(FVector InOffset, bool bWorldShift)
{
	FParticleTrailsEmitterInstance_Base::ApplyWorldOffset(InOffset, bWorldShift);

	for (FVector& Position : CurrentSourcePosition)
	{
		Position += InOffset;
	}
	
	for (FVector& Position : LastSourcePosition)
	{
		Position += InOffset;
	}
}

//
//	FParticleAnimTrailEmitterInstance
//
/** Constructor	*/
FParticleAnimTrailEmitterInstance::FParticleAnimTrailEmitterInstance(IParticleEmitterInstanceOwner& InComponent) :
	  FParticleTrailsEmitterInstance_Base(InComponent)
	, TrailTypeData(NULL)
	, SpawnPerUnitModule(NULL)
	, FirstSocketName(NAME_None)
	, SecondSocketName(NAME_None)
	, Width(1.0f)
	, WidthMode( ETrailWidthMode_FromCentre )
	, Owner(NULL)
	, bTagTrailAsDead(false)
	, bTrailEnabled(false)
#if WITH_EDITORONLY_DATA
	, bRenderGeometry(true)
	, bRenderSpawnPoints(false)
	, bRenderTangents(false)
	, bRenderTessellation(false)
#endif
	, HeadOnlyParticles(0)
{
}

/** Destructor	*/
FParticleAnimTrailEmitterInstance::~FParticleAnimTrailEmitterInstance()
{
}

void FParticleAnimTrailEmitterInstance::InitParameters(UParticleEmitter* InTemplate)
{
	FParticleTrailsEmitterInstance_Base::InitParameters(InTemplate);

	// We don't support LOD on trails
	UParticleLODLevel* LODLevel	= InTemplate->GetLODLevel(0);
	check(LODLevel);
	TrailTypeData = CastChecked<UParticleModuleTypeDataAnimTrail>(LODLevel->TypeDataModule);
	check(TrailTypeData);

	bDeadTrailsOnDeactivate = TrailTypeData->bDeadTrailsOnDeactivate;

	TrailSpawnTimes.Empty(1);
	TrailSpawnTimes.AddZeroed(1);
	SourceDistanceTraveled.Empty(1);
	SourceDistanceTraveled.AddZeroed(1);
	TiledUDistanceTraveled.Empty(1);
	TiledUDistanceTraveled.AddZeroed(1);

	//
	VertexCount = 0;
	TriangleCount = 0;
}

/**
 *	Helper function for recalculating tangents and the interpolation parameter for this particle...
 *
 *	@param	PrevParticle		The previous particle in the trail
 *	@param	PrevTrailData		The payload of the previous particle in the trail
 *	@param	CurrParticle		The current particle in the trail
 *	@param	CurrTrailData		The payload of the current particle in the trail
 *	@param	NextParticle		The next particle in the trail
 *	@param	NextTrailData		The payload of the next particle in the trail
 */
void FParticleAnimTrailEmitterInstance::RecalculateTangentAndInterpolationParam(
	FBaseParticle* PrevParticle, FAnimTrailTypeDataPayload* PrevTrailData, 
	FBaseParticle* CurrParticle, FAnimTrailTypeDataPayload* CurrTrailData, 
	FBaseParticle* NextParticle, FAnimTrailTypeDataPayload* NextTrailData)
{
	check(CurrParticle);
	check(PrevParticle || NextParticle);
	FVector Tangent;
	float SegmentDistance;
	
	if( PrevParticle )
	{
		(PrevParticle->Location - CurrParticle->Location).ToDirectionAndLength(Tangent, SegmentDistance);
		//If there is a next particle and a prev available then we can get a better tangent.
		if (NextParticle != NULL)
		{
			Tangent = PrevParticle->Location - NextParticle->Location;
			if( !Tangent.IsNearlyZero() )
			{
				Tangent.Normalize();
			}
		}
	}
	else
	{
		//Only the next available, this is the head of the trail.
		(CurrParticle->Location - NextParticle->Location).ToDirectionAndLength(Tangent, SegmentDistance);
	}
		
	CurrTrailData->InterpolationParameter = FMath::Sqrt(SegmentDistance);//Using centripetal as it is visually better and can be bounded more conveniently.
	CurrTrailData->Tangent = (FVector3f)Tangent;
}

/**
 *	Tick sub-function that handles recalculation of tangents
 *
 *	@param	DeltaTime			The current time slice
 *	@param	CurrentLODLevel		The current LOD level for the instance
 */
void FParticleAnimTrailEmitterInstance::Tick_RecalculateTangents(float DeltaTime, UParticleLODLevel* InCurrentLODLevel)
{
	if (TrailTypeData->bTangentRecalculationEveryFrame == true)
	{
		//@todo. Multiple trails, single emitter
		int32 TrailIdx = 0;
		int32 StartIndex = 0;
		// Find the Start particle of the current trail...
		FBaseParticle* StartParticle = NULL;
		FAnimTrailTypeDataPayload* StartTrailData = NULL;
		GetTrailStart<FAnimTrailTypeDataPayload>(TrailIdx, StartIndex, StartTrailData, StartParticle);

		// Recalculate tangents at each particle to properly handle moving particles...
		if ((StartParticle != NULL) && (TRAIL_EMITTER_IS_ONLY(StartTrailData->Flags) == 0))
		{
			// For trails, particles go:
			//     START, next, next, ..., END
			// Coming from the end,
			//     END, prev, prev, ..., START
			FBaseParticle* PrevParticle = StartParticle;
			FAnimTrailTypeDataPayload* PrevTrailData = StartTrailData;
			FBaseParticle* CurrParticle = NULL;
			FAnimTrailTypeDataPayload* CurrTrailData = NULL;
			FBaseParticle* NextParticle = NULL;
			FAnimTrailTypeDataPayload* NextTrailData = NULL;

			FTrailsBaseTypeDataPayload* TempPayload = NULL;

			GetParticleInTrail(true, PrevParticle, PrevTrailData, GET_Next, GET_Any, CurrParticle, TempPayload);
			CurrTrailData = (FAnimTrailTypeDataPayload*)(TempPayload);
			while (CurrParticle != NULL)
			{
				if (CheckForCircularTrail(StartParticle, CurrParticle))
				{
					break;
				}

				// Grab the next particle in the trail...
				GetParticleInTrail(true, CurrParticle, CurrTrailData, GET_Next, GET_Any, NextParticle, TempPayload);
				NextTrailData = (FAnimTrailTypeDataPayload*)(TempPayload);

				check(CurrParticle != PrevParticle);
				check(CurrParticle != NextParticle);

				RecalculateTangentAndInterpolationParam(PrevParticle, PrevTrailData, CurrParticle, CurrTrailData, NextParticle, NextTrailData);

				// Move up the chain...
				PrevParticle = CurrParticle;
				PrevTrailData = CurrTrailData;
				CurrParticle = NextParticle;
				CurrTrailData = NextTrailData;
			}
		}
	}
}

bool FParticleAnimTrailEmitterInstance::GetSpawnPerUnitAmount(float DeltaTime, int32 InTrailIdx, int32& OutCount, float& OutRate)
{
	return false;
}

struct FAnimTrailParticleSpawnParams
{
	/** The index of the 'oldest' particle in the current batch of spawns. */
	int32 FirstSpawnIndex;
	/** The index of the current particle being spawned offset from the FirstSpawnParticle*/
	int32 SpawnIndex;
	/** Inverse of the number of particles being spawned. */
	int32 InvCount;
	/** Frame delta time */
	float DeltaTime;
	/** Total elapsed time for this emitter. */
	float ElapsedTime;
	/** Previous elapsed time for this emitter. */
	float LastTime;
	/** ElapsedTime - LastTime */
	float TimeDiff;
	bool bTilingTrail;
	
	
	//TODO - These params are for interpolated spawn particles which are currently disabled.
	///** The GetComponentTransform() transform of the particle component before spawning began. */
	//FTransform SavedComponentToWorld;
	///** True if this particle is interpolated. False otherwise. */
	//bool bInterpolated;
	
	FAnimTrailParticleSpawnParams(	int32 InFirstSpawnIndex, int32 InSpawnIndex, int32 InInvCount, float InDeltaTime, float InElapsedTime,
									float InLastTime, float InTimeDiff, bool InbTilingTrail )
		: 	FirstSpawnIndex(InFirstSpawnIndex), SpawnIndex(InSpawnIndex), InvCount(InInvCount), DeltaTime(InDeltaTime), ElapsedTime(InElapsedTime),
			LastTime(InLastTime), TimeDiff(InTimeDiff), bTilingTrail(InbTilingTrail)
	{
	}
};

void FParticleAnimTrailEmitterInstance::SpawnParticle( int32& StartParticleIndex, const FAnimTrailParticleSpawnParams& Params )
{
	/** Interpolation factor for the current particle between the non interpolated particles either side of it. */
	float InterpFactor = 1.0f;

//	At the moment, spawning interpolated particles is not enabled. 
//	More thought needed.
//	//Calculate interp factor and interpolate the component transform. This is restored back to the SavedComponentTransform after spawning.
//	if( Params.bInterpolated )
//	{
//		InterpFactor = Params.InvCount * Params.SpawnIndex;
//
//		//Lerp this. Desirable to do a spline interpolation?
//		FVector InterpSourcePos = FMath::Lerp<FVector>(PreviousComponentTransform.GetLocation(), Params.SavedComponentToWorld.GetLocation(), InterpFactor);
//		FQuat InterpSourceRot = FQuat::Slerp(PreviousComponentTransform.GetRotation(), Params.SavedComponentToWorld.GetRotation(), InterpFactor);
//		FTransform InterpSourceTransform = FTransform(InterpSourceRot, InterpSourcePos);
//		Component->GetComponentTransform() = InterpSourceTransform;
//	}

	//TODO - Multiple trails.
	int32 TrailIdx = 0;

	UParticleLODLevel* LODLevel = SpriteTemplate->LODLevels[0];
	int32 ParticleIndex = ParticleIndices[Params.FirstSpawnIndex+Params.SpawnIndex];
	DECLARE_PARTICLE_PTR(Particle, ParticleData + ParticleStride * ParticleIndex);
	FAnimTrailTypeDataPayload* TrailData = ((FAnimTrailTypeDataPayload*)((uint8*)Particle + TypeDataOffset));

	FBaseParticle* StartParticle = NULL;
	FAnimTrailTypeDataPayload* StartTrailData = NULL;

	float SpawnTime = Params.DeltaTime * InterpFactor;
	float SpawnTimeDelta = Params.TimeDiff * InterpFactor;

	// Standard spawn setup
	PreSpawn(Particle, Location, FVector::ZeroVector);
	SetDeadIndex(TrailData->TrailIndex, ParticleIndex);
	for (int32 SpawnModuleIdx = 0; SpawnModuleIdx < LODLevel->SpawnModules.Num(); SpawnModuleIdx++)
	{
		UParticleModule* SpawnModule = LODLevel->SpawnModules[SpawnModuleIdx];
		if (!SpawnModule || !SpawnModule->bEnabled)
		{
			continue;
		}

		SpawnModule->Spawn({ *this, (int32)GetModuleDataOffset(SpawnModule), SpawnTime, Particle });
	}

	if ((1.0f / Particle->OneOverMaxLifetime) < 0.001f)
	{
		Particle->OneOverMaxLifetime = 1.f / 0.001f;
	}

	if (LODLevel->TypeDataModule)
	{
		//@todo. Need to track TypeData offset into payload!
		LODLevel->TypeDataModule->Spawn({ *this, TypeDataOffset, SpawnTime, Particle });
	}
	PostSpawn(Particle, 1.f, SpawnTime);

	//Now init the particle values and flags
//	if( Params.bInterpolated )
//	{
		//Interpolated particles are not enabled currently. All interpolation is done during vertex generation.
		// Not too sure interpolating using catmull-rom splines is going to work really given that we don't have a "next" particle to influence the interpolation.
		// Making it up may work ok for rendering as that data is regenerated correctly when there is data to use but actually filling in duff data like this seems bad.
//	}
//	else
	{
		UMeshComponent* MeshComp = Cast<UMeshComponent>(Component.AsComponent()->GetAttachParent());
		check(TrailTypeData);
		check(MeshComp);

		//This particle samples the animated socket locations.
		FTransform FirstEdgeSocketSample = MeshComp->GetSocketTransform(FirstSocketName);
		FTransform SecondEdgeSocketSample = MeshComp->GetSocketTransform(SecondSocketName);
		
		// Trail specific...
		// Clear the next and previous - just to be safe
		TrailData->Flags = TRAIL_EMITTER_SET_NEXT(TrailData->Flags, TRAIL_EMITTER_NULL_NEXT);
		TrailData->Flags = TRAIL_EMITTER_SET_PREV(TrailData->Flags, TRAIL_EMITTER_NULL_PREV);
		// Set the trail-specific data on this particle
		TrailData->TrailIndex = TrailIdx;
		TrailData->SpawnTime = Params.LastTime + SpawnTimeDelta;
 		
		//Not being used at the moment for anim trails or ribbons it seems. Can be removed?
		TrailData->SpawnDelta = Params.SpawnIndex * InterpFactor;

		// Set the location
		TrailData->bInterpolatedSpawn = false; 
		TrailData->SpawnedTessellationPoints = 1;

		FVector First = FirstEdgeSocketSample.GetLocation();
		FVector Second = SecondEdgeSocketSample.GetLocation();
		FVector Dir;
		float Length;

		(Second - First).ToDirectionAndLength(Dir, Length);
		//Particle location is in the center of the sheet (for better tangent calcs and interpolation).
		switch ((uint32)WidthMode)
		{
		case ETrailWidthMode_FromCentre:
		{
			Length *= 0.5f;
			First += (Dir * Length);
			Length *= Width;
		}
			break;
		case ETrailWidthMode_FromFirst:
		{
			Length *= 0.5f * Width;
			First += (Dir * Length);
		}
			break;
		case ETrailWidthMode_FromSecond:
		{
			Length *= 0.5f * Width;
			First = Second - (Dir * Length);
		}
			break;
		default:
			UE_LOGF(LogParticles, Fatal, "Invalid Width Mode for trail: %u", (uint32)WidthMode);
			break;
		}
		Particle->Location = First;
		Particle->OldLocation = First;
		TrailData->Direction = (FVector3f)Dir;
		TrailData->Length = Length;

		bool bAddedParticle = false;
		//Todo - Multiple trails.
		if (INDEX_NONE == StartParticleIndex)
		{
			// This it the first particle.
			// Tag it as the 'only'
			TrailData->Flags = TRAIL_EMITTER_SET_ONLY(TrailData->Flags);
			TrailData->Flags = TRAIL_EMITTER_SET_PREV(TrailData->Flags, TRAIL_EMITTER_NULL_PREV);
			TrailData->Flags = TRAIL_EMITTER_SET_NEXT(TrailData->Flags, TRAIL_EMITTER_NULL_NEXT);
			TiledUDistanceTraveled[TrailIdx] = 0.0f;
			TrailData->TiledU = 0.0f;
			bAddedParticle		= true;
			TrailData->InterpolationParameter = 0.0f;
			SetStartIndex(TrailData->TrailIndex, ParticleIndex);
		}
		else
		{
			StartParticle = (FBaseParticle*)(ParticleData + ParticleStride * StartParticleIndex);
			StartTrailData = ((FAnimTrailTypeDataPayload*)((uint8*)StartParticle + TypeDataOffset));

			//@todo. Determine how to handle multiple trails...
			if (TRAIL_EMITTER_IS_ONLY(StartTrailData->Flags))
			{
				StartTrailData->Flags	= TRAIL_EMITTER_SET_END(StartTrailData->Flags);
				StartTrailData->Flags	= TRAIL_EMITTER_SET_NEXT(StartTrailData->Flags, TRAIL_EMITTER_NULL_NEXT);
				StartTrailData->Flags	= TRAIL_EMITTER_SET_PREV(StartTrailData->Flags, ParticleIndex);
				SetEndIndex(StartTrailData->TrailIndex, StartParticleIndex);

				if (TrailData->SpawnTime < StartTrailData->SpawnTime)
				{
					UE_LOGF(LogParticles, Log, "BAD SPAWN TIME! Curr %8.6f, Start %8.6f", TrailData->SpawnTime, StartTrailData->SpawnTime);
				}

				// Now, 'join' them
				TrailData->Flags		= TRAIL_EMITTER_SET_PREV(TrailData->Flags, TRAIL_EMITTER_NULL_PREV);
				TrailData->Flags		= TRAIL_EMITTER_SET_NEXT(TrailData->Flags, StartParticleIndex);
				TrailData->Flags		= TRAIL_EMITTER_SET_START(TrailData->Flags);
				SetStartIndex(TrailData->TrailIndex, ParticleIndex);

				//Try to get a half passable tangent for the current particle.					
				RecalculateTangentAndInterpolationParam(NULL,NULL,Particle,TrailData,StartParticle,StartTrailData);

				//As this is the second particle in the trail we also have to regenerate the initial particle tangent and interpolation parameter.
				RecalculateTangentAndInterpolationParam(Particle,TrailData,StartParticle,StartTrailData,NULL,NULL);

				bAddedParticle = true;
			}
			else
			{
				// It better be the start!!!
				check(TRAIL_EMITTER_IS_START(StartTrailData->Flags));
				check(TRAIL_EMITTER_GET_NEXT(StartTrailData->Flags) != TRAIL_EMITTER_NULL_NEXT);

				StartTrailData->Flags	= TRAIL_EMITTER_SET_MIDDLE(StartTrailData->Flags);
				StartTrailData->Flags	= TRAIL_EMITTER_SET_PREV(StartTrailData->Flags, ParticleIndex);
				ClearIndices(StartTrailData->TrailIndex, StartParticleIndex);

				if (TrailData->SpawnTime < StartTrailData->SpawnTime)
				{
					checkf(0, TEXT("BAD SPAWN TIME! Curr %8.6f, Start %8.6f"), TrailData->SpawnTime, StartTrailData->SpawnTime);
				}

				// Now, 'join' them
				TrailData->Flags		= TRAIL_EMITTER_SET_PREV(TrailData->Flags, TRAIL_EMITTER_NULL_PREV);
				TrailData->Flags		= TRAIL_EMITTER_SET_NEXT(TrailData->Flags, StartParticleIndex);
				TrailData->Flags		= TRAIL_EMITTER_SET_START(TrailData->Flags);

				SetStartIndex(TrailData->TrailIndex, ParticleIndex);

				bAddedParticle = true;

				//Try to get a half passable tangent for the current particle.					
				RecalculateTangentAndInterpolationParam(NULL,NULL,Particle,TrailData,StartParticle,StartTrailData);
			}

			if ((TrailTypeData->bEnablePreviousTangentRecalculation == true)&& 
				(TrailTypeData->bTangentRecalculationEveryFrame == false))
			{
				FBaseParticle* PrevParticle = Particle;
				FAnimTrailTypeDataPayload* PrevTrailData = TrailData;
				FBaseParticle* CurrParticle = StartParticle;
				FAnimTrailTypeDataPayload* CurrTrailData = StartTrailData;
				FBaseParticle* NextParticle = NULL;
				FAnimTrailTypeDataPayload* NextTrailData = NULL;

				int32 StartNext = TRAIL_EMITTER_GET_NEXT(StartTrailData->Flags);
				if (StartNext != TRAIL_EMITTER_NULL_NEXT)
				{
					DECLARE_PARTICLE_PTR(TempParticle, ParticleData + ParticleStride * StartNext);
					NextParticle = TempParticle;
					NextTrailData = ((FAnimTrailTypeDataPayload*)((uint8*)NextParticle + TypeDataOffset));
				}
				RecalculateTangentAndInterpolationParam(PrevParticle, PrevTrailData, CurrParticle, CurrTrailData, NextParticle, NextTrailData);
				PrevTrailData->Tangent = CurrTrailData->Tangent;
			}
		}

		if( bAddedParticle )
		{
			if (Params.bTilingTrail == true)
			{
				if (INDEX_NONE == StartParticleIndex)
				{
					TrailData->TiledU = 0.0f;
				}
				else
				{
					check(StartParticle);
					check(StartTrailData);

					FVector PositionDelta = Particle->Location - StartParticle->Location;
					TiledUDistanceTraveled[TrailIdx] += PositionDelta.Size();
					TrailData->TiledU = TiledUDistanceTraveled[TrailIdx] / TrailTypeData->TilingDistance;
					//@todo. Is there going to be a problem when distance gets REALLY high?
				}
			}

			StartParticleIndex = ParticleIndex;

			ActiveParticles++;
			//check((int32)ActiveParticles < TrailTypeData->MaxParticleInTrailCount);
			INC_DWORD_STAT(STAT_TrailParticlesSpawned);
		}
		else
		{
			check(TEXT("Failed to add particle to trail!!!!"));
		}

		INC_DWORD_STAT_BY(STAT_TrailParticles, ActiveParticles);
	}
}

float FParticleAnimTrailEmitterInstance::Spawn(float DeltaTime)
{
	static int32 TickCount = 0;

	UParticleLODLevel* LODLevel = SpriteTemplate->LODLevels[0];
	check(LODLevel);
	check(LODLevel->RequiredModule);

	// Iterate over each trail
	//@todo. Add support for multiple trails?
//	for (int32 TrailIdx = 0; TrailIdx < MaxTrailCount; TrailIdx++)
	int32 TrailIdx = 0;

	float SpawnRate = 0.0f;
	int32 BurstCount = 0;
	float OldLeftover = SpawnFraction;
	// For now, we are not supporting bursts on trails...
	bool bProcessSpawnRate = false;
	bool bProcessBurstList = false;

	// Figure out spawn rate for this tick.
	if (bProcessSpawnRate)
	{
		int32 EffectsQuality = Scalability::GetEffectsQualityDirect(true);
		float RateScale = LODLevel->SpawnModule->RateScale.GetValue(EmitterTime, Component.GetDistributionData()) * LODLevel->SpawnModule->GetGlobalRateScale();
		float QualityMult = 0.25f * (1 << EffectsQuality);
		RateScale *= SpriteTemplate->QualityLevelSpawnRateScale*QualityMult;
		SpawnRate += LODLevel->SpawnModule->Rate.GetValue(EmitterTime, Component.GetDistributionData()) * FMath::Clamp<float>(RateScale, 0.0f, RateScale);
	}

	// Take Bursts into account as well...
	if (bProcessBurstList)
	{
		int32 Burst = 0;
		float BurstTime = GetCurrentBurstRateOffset(DeltaTime, Burst);
		BurstCount += Burst;
	}

	float SafetyLeftover = OldLeftover;
	float NewLeftover = OldLeftover + DeltaTime * SpawnRate;
	int32 SpawnNumber	= FMath::FloorToInt(NewLeftover);
	float SliceIncrement = (SpawnRate > 0.0f) ? (1.f / SpawnRate) : 0.0f;
	float SpawnStartTime = DeltaTime + OldLeftover * SliceIncrement - SliceIncrement;
	SpawnFraction = NewLeftover - SpawnNumber;
	// Do the resize stuff here!!!!!!!!!!!!!!!!!!!
	int32 NewCount = 1 + SpawnNumber + BurstCount; //At least 1 for the actual anim sample.

	//TODO - Sort this out. If max particles is exceeded maybe mark the trail as dead? Or prematurely kill off old particles?
	// Don't allow more than TrailCount trails...
// 	int32	MaxParticlesAllowed	= MaxTrailCount * TrailTypeData->MaxParticleInTrailCount;
// 	if ((TotalCount + ActiveParticles) > MaxParticlesAllowed)
// 	{
// 		TotalCount = MaxParticlesAllowed - ActiveParticles - 1;
// 		if (TotalCount < 0)
// 		{
// 			TotalCount = 0;
// 		}
// 	}

	// Handle growing arrays.
	bool bProcessSpawn = true;
	int32 TotalCount = ActiveParticles + NewCount;

	constexpr int32 MaxTrailParticles = FMath::Min(TRAIL_EMITTER_NEXT_MASK >> TRAIL_EMITTER_NEXT_SHIFT, TRAIL_EMITTER_PREV_MASK >> TRAIL_EMITTER_PREV_SHIFT);
	if (TotalCount >= MaxTrailParticles)
	{
		// Note the above test is >= as null is considered == max value
		bProcessSpawn = false;
	}
	else if (TotalCount >= MaxActiveParticles)
	{
		if (DeltaTime < 0.25f)
		{
			bProcessSpawn = Resize(TotalCount + FMath::TruncToInt(FMath::Sqrt((float)TotalCount)) + 1);
		}
		else
		{
			bProcessSpawn = Resize((TotalCount + FMath::TruncToInt(FMath::Sqrt((float)TotalCount)) + 1), false);
		}
	}

	if (bProcessSpawn == false)
	{
		return SafetyLeftover;
	}

	//@todo. Support multiple trails per emitter
	// Find the start particle of the current trail...
	int32 StartIndex = -1;
	if (TrailIdx != INDEX_NONE) //-V547
	{
		FBaseParticle* Particle = nullptr;
		FAnimTrailTypeDataPayload* TrailData = nullptr;
		GetTrailStart(TrailIdx, StartIndex, TrailData, Particle);
	}

	bool bTilingTrail = !FMath::IsNearlyZero(TrailTypeData->TilingDistance);

	//The mesh we're sampline socket locations from.
	UMeshComponent* MeshComp = Cast<UMeshComponent>(Component.AsComponent()->GetAttachParent());

	check(TrailTypeData);
	// Don't allow new spawning if the emitter is finished
	if (MeshComp && NewCount > 0 && bTrailEnabled)
	{
		FAnimTrailParticleSpawnParams SpawnParams( 
			ActiveParticles,
			NewCount-1,
			1.0f / NewCount,
			DeltaTime,
			RunningTime,
			TrailSpawnTimes[TrailIdx],
			RunningTime - TrailSpawnTimes[TrailIdx],
			bTilingTrail);

		//TODO - For interpolated spawn which is disabled currently.
		//if(ActiveParticles > 0)
		//{
			//PreviousComponentTransform = SpawnParams.SavedComponentToWorld;
		//}
		
		float ProcessedTime = 0.0f;

		//Spawn sampled particle at the end.
		//SpawnParams.bInterpolated = false;
		SpawnParticle(StartIndex, SpawnParams);
		INC_DWORD_STAT_BY(STAT_TrailParticles, ActiveParticles);
		
//		Interpolated Spawned Particles are Disabled For Now.
//		Needs more thought but some kind of interpolation here will be needed to avoid DeltaTime depenance on spawn positions.
//		//Work forwards from the oldest particle laid up to the final particle which will be at the sample position.
//		SpawnParams.bInterpolated = true;
//		for (int32 SpawnIdx = 0; SpawnIdx < NewCount-1; SpawnIdx++)
//		{
//			SpawnParams.SpawnIndex = SpawnIdx;
//
//			SpawnParticle( StartParticleIndex, SpawnParams );
//
//			INC_DWORD_STAT_BY(STAT_TrailParticles, ActiveParticles);
//		}
	}

	// Update the last position
	TrailSpawnTimes[TrailIdx] = RunningTime;

	//TODO - If I enable interpolated spawning then the component transform needs restoring and storing.
	//Component->GetComponentTransform() = SpawnParams.SavedComponentToWorld;
	//PreviousComponentTransform = Component->GetComponentTransform();

	if (bTagTrailAsDead == true)
	{
		for (int32 FindTrailIdx = 0; FindTrailIdx < ActiveParticles; FindTrailIdx++)
		{
			int32 CheckStartIndex = ParticleIndices[FindTrailIdx];
			DECLARE_PARTICLE_PTR(CheckParticle, ParticleData + ParticleStride * CheckStartIndex);
			FAnimTrailTypeDataPayload* CheckTrailData = ((FAnimTrailTypeDataPayload*)((uint8*)CheckParticle + TypeDataOffset));
			if (CheckTrailData->TrailIndex == TrailIdx)
			{
				if (TRAIL_EMITTER_IS_START(CheckTrailData->Flags))
				{
					CheckTrailData->Flags = TRAIL_EMITTER_SET_DEADTRAIL(CheckTrailData->Flags);
				}
			}
		}
		bTagTrailAsDead = false;
	}
	TickCount++;
	return SpawnFraction;
}

void FParticleAnimTrailEmitterInstance::SetupTrailModules()
{
	// Trails are a special case... 
	// We don't want standard Spawn/Update calls occurring on Trail-type modules.
	UParticleLODLevel* LODLevel = SpriteTemplate->GetLODLevel(0);
	check(LODLevel);
	for (int32 ModuleIdx = 0; ModuleIdx < LODLevel->Modules.Num(); ModuleIdx++)
	{
		bool bRemoveIt = false;
		UParticleModule* CheckModule = LODLevel->Modules[ModuleIdx];
		UParticleModuleSpawnPerUnit* CheckSPUModule = Cast<UParticleModuleSpawnPerUnit>(CheckModule);
		if (CheckSPUModule != NULL)
		{
			SpawnPerUnitModule = CheckSPUModule;
			bRemoveIt = true;
		}

		if (bRemoveIt == true)
		{
			// Remove it from any lists...
			for (int32 UpdateIdx = LODLevel->UpdateModules.Num() - 1; UpdateIdx >= 0; UpdateIdx--)
			{
				if (LODLevel->UpdateModules[UpdateIdx] == CheckModule)
				{
					LODLevel->UpdateModules.RemoveAt(UpdateIdx);
				}
			}

			for (int32 SpawnIdx = LODLevel->SpawnModules.Num() - 1; SpawnIdx >= 0; SpawnIdx--)
			{
				if (LODLevel->SpawnModules[SpawnIdx] == CheckModule)
				{
					LODLevel->SpawnModules.RemoveAt(SpawnIdx);
				}
			}

			for (int32 SpawningIdx = LODLevel->SpawningModules.Num() - 1; SpawningIdx >= 0; SpawningIdx--)
			{
				if (LODLevel->SpawningModules[SpawningIdx] == CheckModule)
				{
					LODLevel->SpawningModules.RemoveAt(SpawningIdx);
				}
			}
		}
	}
}

void FParticleAnimTrailEmitterInstance::UpdateSourceData(float DeltaTime, bool bFirstTime)
{
}

void FParticleAnimTrailEmitterInstance::UpdateBoundingBox(float DeltaTime)
{
	SCOPE_CYCLE_COUNTER(STAT_ParticleUpdateBounds);
	{
		const UParticleSystem* Template = Component.GetTemplate();
		bool bUpdateBox = ((Component.GetIsWarmingUp() == false) && (Template != NULL) && (Template->bUseFixedRelativeBoundingBox == false));
		// Handle local space usage
		check(SpriteTemplate->LODLevels.Num() > 0);
		UParticleLODLevel* LODLevel = SpriteTemplate->LODLevels[0];
		check(LODLevel);

		if (bUpdateBox)
		{
			// Set the min/max to the position of the trail
			if (LODLevel->RequiredModule->bUseLocalSpace == false) 
			{
				ParticleBoundingBox.Max = Component.GetComponentTransform().GetLocation();
				ParticleBoundingBox.Min = Component.GetComponentTransform().GetLocation();
			}
			else
			{
				ParticleBoundingBox.Max = FVector::ZeroVector;
				ParticleBoundingBox.Min = FVector::ZeroVector;
			}
		}
		ParticleBoundingBox.IsValid = true;

		// Take scale into account
		FVector Scale = Component.GetComponentTransform().GetScale3D();

		// As well as each particle
		int32 LocalActiveParticles = ActiveParticles;
		if (LocalActiveParticles > 0)
		{
			FVector MinPos(FLT_MAX);
			FVector MaxPos(-FLT_MAX);
			FVector TempMin;
			FVector TempMax;

			DECLARE_PARTICLE_PTR(FirstParticle, ParticleData + ParticleStride * ParticleIndices[0]);
			FAnimTrailTypeDataPayload*	FirstPayload = ((FAnimTrailTypeDataPayload*)((uint8*)FirstParticle + TypeDataOffset));
			FVector PrevParticleLocation = FirstParticle->Location + PositionOffsetThisTick;
			float PrevParticleLength = FirstPayload->Length;

			bool bSkipDoubleSpawnUpdate = !SpriteTemplate->bUseLegacySpawningBehavior;
			for (int32 i = 0; i < LocalActiveParticles; i++)
			{
				DECLARE_PARTICLE_PTR(Particle, ParticleData + ParticleStride * ParticleIndices[i]);
				FAnimTrailTypeDataPayload*	Payload = ((FAnimTrailTypeDataPayload*)((uint8*)Particle + TypeDataOffset));

				bool bJustSpawned = (Particle->Flags & STATE_Particle_JustSpawned) != 0;
				Particle->Flags &= ~STATE_Particle_JustSpawned;

				//Don't update position for newly spawned particles. They already have a partial update applied during spawn.
				bool bSkipUpdate = bJustSpawned && bSkipDoubleSpawnUpdate;
				// Do linear integrator and update bounding box
				Particle->Location	+= bSkipUpdate ? FVector::ZeroVector : FVector(DeltaTime * Particle->Velocity);
				Particle->Rotation	+= bSkipUpdate ? 0.0f : DeltaTime * Particle->RotationRate;
				Particle->Location	+= PositionOffsetThisTick;
				
				FPlatformMisc::Prefetch(ParticleData, (ParticleIndices[i+1] * ParticleStride));
				FPlatformMisc::Prefetch(ParticleData, (ParticleIndices[i+1] * ParticleStride) + PLATFORM_CACHE_LINE_SIZE);
				Particle->OldLocation = Particle->Location;
				if (bUpdateBox)
				{
					//Interpolated points on the trail can be bounded by 1/4 the length of the segment for centripetal parameterization.	
					//Length is also interpolated so need to use the maximum possible length. TODO - Handle it dropping to zero or -ve?
					//This makes the bounds quite a bit fatter than they need to be. Especially for wide trails. I expect it can be improved quite a bit.
					float LengthBound = Payload->Length + ((Payload->Length - PrevParticleLength) * .25f);		
					FVector BoundSize = FVector( ((Particle->Location - PrevParticleLocation).Size() * 0.25f) + LengthBound);
					PrevParticleLocation = Particle->Location;
					PrevParticleLength = Payload->Length;

					TempMin = Particle->Location - BoundSize;
					TempMax = Particle->Location + BoundSize;

					MinPos.X = FMath::Min(TempMin.X, MinPos.X);
					MinPos.Y = FMath::Min(TempMin.Y, MinPos.Y);
					MinPos.Z = FMath::Min(TempMin.Z, MinPos.Z);
					MaxPos.X = FMath::Max(TempMin.X, MaxPos.X);
					MaxPos.Y = FMath::Max(TempMin.Y, MaxPos.Y);
					MaxPos.Z = FMath::Max(TempMin.Z, MaxPos.Z);
					MinPos.X = FMath::Min(TempMax.X, MinPos.X);
					MinPos.Y = FMath::Min(TempMax.Y, MinPos.Y);
					MinPos.Z = FMath::Min(TempMax.Z, MinPos.Z);
					MaxPos.X = FMath::Max(TempMax.X, MaxPos.X);
					MaxPos.Y = FMath::Max(TempMax.Y, MaxPos.Y);
					MaxPos.Z = FMath::Max(TempMax.Z, MaxPos.Z);
				}

				// Do angular integrator, and wrap result to within +/- 2 PI
				Particle->Rotation	 = FMath::Fmod(Particle->Rotation, 2.f*(float)UE_PI);
			}
			if (bUpdateBox)
			{
				ParticleBoundingBox += MinPos;
				ParticleBoundingBox += MaxPos;
			}
		}

		// Transform bounding box into world space if the emitter uses a local space coordinate system.
		if (bUpdateBox)
		{
			if (LODLevel->RequiredModule->bUseLocalSpace) 
			{
				ParticleBoundingBox = ParticleBoundingBox.TransformBy(Component.GetComponentTransform());
			}
		}
	}
}

/**
 * Force the bounding box to be updated.
 */
void FParticleAnimTrailEmitterInstance::ForceUpdateBoundingBox()
{
	{
		// Handle local space usage
		check(SpriteTemplate->LODLevels.Num() > 0);
		UParticleLODLevel* LODLevel = SpriteTemplate->LODLevels[0];
		check(LODLevel);

		// Set the min/max to the position of the trail
		if (LODLevel->RequiredModule->bUseLocalSpace == false) 
		{
			ParticleBoundingBox.Max = Component.GetComponentTransform().GetLocation();
			ParticleBoundingBox.Min = Component.GetComponentTransform().GetLocation();
		}
		else
		{
			ParticleBoundingBox.Max = FVector::ZeroVector;
			ParticleBoundingBox.Min = FVector::ZeroVector;
		}
		ParticleBoundingBox.IsValid = true;

		// Take scale into account
		FVector Scale = Component.GetComponentTransform().GetScale3D();

		// As well as each particle
		int32 LocalActiveParticles = ActiveParticles;
		if (LocalActiveParticles > 0)
		{
			FVector MinPos(FLT_MAX);
			FVector MaxPos(-FLT_MAX);
			FVector TempMin;
			FVector TempMax;			

			DECLARE_PARTICLE_PTR(FirstParticle, ParticleData + ParticleStride * ParticleIndices[0]);
			FAnimTrailTypeDataPayload*	FirstPayload = ((FAnimTrailTypeDataPayload*)((uint8*)FirstParticle + TypeDataOffset));
			FVector PrevParticleLocation = FirstParticle->Location;
			float PrevParticleLength = FirstPayload->Length;
			for (int32 i = 0; i < LocalActiveParticles; i++)
			{
				DECLARE_PARTICLE_PTR(Particle, ParticleData + ParticleStride * ParticleIndices[i]);
				FAnimTrailTypeDataPayload*	Payload = ((FAnimTrailTypeDataPayload*)((uint8*)Particle + TypeDataOffset));
				
				//Interpolated points on the trail can be bounded by 1/4 the length of the segment for centripetal parameterization.	
				//Length is also interpolated so need to use the maximum possible length. TODO - Handle it dropping to zero or -ve?
				//This makes the bounds quite a bit fatter than they need to be. Especially for wide trails. I expect it can be improved quite a bit.
				float LengthBound = Payload->Length + ((Payload->Length - PrevParticleLength) * .25f);		
				FVector BoundSize = FVector( ((Particle->Location - PrevParticleLocation).Size() * 0.25f) + LengthBound);
				PrevParticleLocation = Particle->Location;
				PrevParticleLength = Payload->Length;

				TempMin = Particle->Location - BoundSize;
				TempMax = Particle->Location + BoundSize;

				MinPos.X = FMath::Min(TempMin.X, MinPos.X);
				MinPos.Y = FMath::Min(TempMin.Y, MinPos.Y);
				MinPos.Z = FMath::Min(TempMin.Z, MinPos.Z);
				MaxPos.X = FMath::Max(TempMin.X, MaxPos.X);
				MaxPos.Y = FMath::Max(TempMin.Y, MaxPos.Y);
				MaxPos.Z = FMath::Max(TempMin.Z, MaxPos.Z);
				MinPos.X = FMath::Min(TempMax.X, MinPos.X);
				MinPos.Y = FMath::Min(TempMax.Y, MinPos.Y);
				MinPos.Z = FMath::Min(TempMax.Z, MinPos.Z);
				MaxPos.X = FMath::Max(TempMax.X, MaxPos.X);
				MaxPos.Y = FMath::Max(TempMax.Y, MaxPos.Y);
				MaxPos.Z = FMath::Max(TempMax.Z, MaxPos.Z);
			}

			ParticleBoundingBox += MinPos;
			ParticleBoundingBox += MaxPos;
		}

		// Transform bounding box into world space if the emitter uses a local space coordinate system.
		if (LODLevel->RequiredModule->bUseLocalSpace) 
		{
			ParticleBoundingBox = ParticleBoundingBox.TransformBy(Component.GetComponentTransform());
		}
	}
}

/** Determine the number of vertices and triangles in each trail */
void FParticleAnimTrailEmitterInstance::DetermineVertexAndTriangleCount()
{
	uint32 NewSize = 0;
	int32 Sheets = 1;//TrailTypeData->Sheets ? TrailTypeData->Sheets : 1;
	int32 TheTrailCount = 0;
	int32 IndexCount = 0;

	VertexCount		= 0;
	TriangleCount	= 0;
	HeadOnlyParticles = 0;

	int32 CheckParticleCount = 0;
	int32 TempVertexCount;

	bool bApplyDistanceTessellation = !FMath::IsNearlyZero(TrailTypeData->DistanceTessellationStepSize);
	bool bApplyTangentTessellation = !FMath::IsNearlyZero(TrailTypeData->TangentTessellationStepSize);
	bool bApplyWidthTessellation = !FMath::IsNearlyZero(TrailTypeData->WidthTessellationStepSize);

	//There's little point doing this if a decent tangent isn't being calculated. Todo - Expose this to user?
	bool bUseNextInTangetTesselationCalculations = TrailTypeData->bEnablePreviousTangentRecalculation;
	bool bUseNextInWidthTesselationCalculations = true;

	float TangentTessellationStepSize = FMath::Fmod(TrailTypeData->TangentTessellationStepSize, 180.0f) / 180.0f;
	
	for (int32 ii = 0; ii < ActiveParticles; ii++)
	{
		int32 LocalIndexCount = 0;
		int32	ParticleCount = 0;
		int32 LocalVertexCount = 0;
		int32 LocalTriCount = 0;

		bool bProcessParticle = false;

		FBaseParticle* NextParticle = NULL;
		FAnimTrailTypeDataPayload* NextTrailData = NULL;

		DECLARE_PARTICLE_PTR(Particle, ParticleData + ParticleStride * ParticleIndices[ii]);
		FBaseParticle* CurrParticle = Particle;
		FAnimTrailTypeDataPayload*	CurrTrailData = ((FAnimTrailTypeDataPayload*)((uint8*)Particle + TypeDataOffset));
		if (TRAIL_EMITTER_IS_HEADONLY(CurrTrailData->Flags))
		{
			CurrTrailData->RenderingInterpCount = 0;
			CurrTrailData->TriangleCount = 0;
			++HeadOnlyParticles;
		}
		else if (TRAIL_EMITTER_IS_END(CurrTrailData->Flags))
		{
			// Walk from the end of the trail to the front
			FBaseParticle* PrevParticle = NULL;
			FAnimTrailTypeDataPayload*	PrevTrailData = NULL;
			int32	Prev = TRAIL_EMITTER_GET_PREV(CurrTrailData->Flags);
			if (Prev != TRAIL_EMITTER_NULL_PREV)
			{
				DECLARE_PARTICLE_PTR(InnerParticle, ParticleData + ParticleStride * Prev);
				PrevParticle = InnerParticle;
				PrevTrailData = ((FAnimTrailTypeDataPayload*)((uint8*)InnerParticle + TypeDataOffset));

				bool bDone = false;
				// The end of the trail, so there MUST be another particle
				while (!bDone)
				{
					if (CheckForCircularTrail(Particle, PrevParticle))
					{
						break;
					}

					ParticleCount++;
					int32 InterpCount = 1;
					if( bApplyDistanceTessellation )
					{
						// Determine the number of rendered interpolated points between these two particles
						float CheckDistance = (CurrParticle->Location - PrevParticle->Location).Size();
						float DistDiff = CheckDistance / TrailTypeData->DistanceTessellationStepSize;
						InterpCount += FMath::Min(FMath::TruncToInt(DistDiff), FXConsoleVariables::MaxDistanceTessellation);
					}
					
					if (bApplyTangentTessellation )
					{
						//Alter tessellation based on difference between tangents.
						float CheckTangent = (CurrTrailData->Tangent | PrevTrailData->Tangent);
						// Map the tangent difference to [0..1] for [0..180]
						//  1.0 = parallel    --> -1 = 0
						//  0.0 = orthogonal  --> -1 = -1 --> * -0.5 = 0.5
						// -1.0 = oppositedir --> -1 = -2 --> * -0.5 = 1.0
						CheckTangent = (CheckTangent - 1.0f) * -0.5f;		
						if( bUseNextInTangetTesselationCalculations && NextTrailData )
						{
							float NextCheckTangent = (NextTrailData->Tangent | CurrTrailData->Tangent);
							NextCheckTangent = (NextCheckTangent - 1.0f) * -0.5f;		
							CheckTangent = FMath::Max( CheckTangent, NextCheckTangent );
						}		
						float TangDiff = CheckTangent / TangentTessellationStepSize;
						InterpCount += FMath::Min(FMath::TruncToInt(TangDiff), FXConsoleVariables::MaxTangentTessellation);
					}

					if( bApplyWidthTessellation )
					{
						//Alter the tessellation based on the change in the trail width.
						float CheckWidth = FMath::Abs(CurrTrailData->Length - PrevTrailData->Length);
						if (bUseNextInWidthTesselationCalculations && NextTrailData)
						{
							CheckWidth = FMath::Max( CheckWidth, FMath::Abs(NextTrailData->Length - CurrTrailData->Length) );
						}
						float WidthDiff = CheckWidth / TrailTypeData->WidthTessellationStepSize;
						InterpCount += FMath::TruncToInt(WidthDiff);
					}

					// Store off the rendering interp count for this particle
					CurrTrailData->RenderingInterpCount = InterpCount;

					// Tally up the vertex and index counts for this segment...
					TempVertexCount = 2 * InterpCount * Sheets;
					VertexCount += TempVertexCount;
					LocalVertexCount += TempVertexCount;
					LocalIndexCount += TempVertexCount;

					// Move to the previous particle in the chain
					NextParticle = CurrParticle;
					NextTrailData = CurrTrailData;
					CurrParticle = PrevParticle;
					CurrTrailData = PrevTrailData;
					Prev = TRAIL_EMITTER_GET_PREV(CurrTrailData->Flags);
					if (Prev != TRAIL_EMITTER_NULL_PREV)
					{
						DECLARE_PARTICLE_PTR(PrevParticlePtr, ParticleData + ParticleStride * Prev);
						PrevParticle = PrevParticlePtr;
						PrevTrailData = ((FAnimTrailTypeDataPayload*)((uint8*)PrevParticle + TypeDataOffset));
					}
					else
					{
						// The START particle will have a previous index of NULL, so we're done
						bDone = true;
					}
				}

				bProcessParticle = true;
			}
			else
			{
				// This means there is only a single particle in the trail - the end...
// 				check(!TEXT("FAIL"));
				bProcessParticle = false;
			}
		}

		if (bProcessParticle == true)
		{
			// The last step is the last interpolated step to the Curr (which should be the start)
			ParticleCount++;
			TempVertexCount = 2 * Sheets;
			VertexCount += TempVertexCount;
			LocalVertexCount += TempVertexCount;
			LocalIndexCount += TempVertexCount;

			// If we are running up to the current source, take that into account as well.
// 			if (TrailTypeData->bClipSourceSegement == false)
// 			{
// 				//@todo. We should interpolate to the source as well!
// 				TempVertexCount = 2 * Sheets;
// 				VertexCount += TempVertexCount;
// 				LocalIndexCount += TempVertexCount;
// 			}

			// Handle degenerates - 4 tris per stitch
			LocalIndexCount	+= ((Sheets - 1) * 4);

			// @todo: We're going and modifying the original ParticleData here!  This is kind of sketchy
			//    since it's not supposed to be changed at this phase
			//check(TRAIL_EMITTER_IS_HEAD(CurrTrailData->Flags));
			CurrTrailData->TriangleCount = LocalIndexCount - 2;

			// The last particle in the chain will always have 1 here!
			CurrTrailData->RenderingInterpCount = 1;

			// Increment the total index count
			IndexCount += LocalIndexCount;
			TheTrailCount++;
		}
	}

	TrailCount = TheTrailCount;
	if (TheTrailCount > 0)
	{
		IndexCount += 4 * (TheTrailCount - 1);	// 4 extra indices per Trail (degenerates)
		TriangleCount = IndexCount - (2 * TheTrailCount);
	}
	else
	{
		IndexCount = 0;
		TriangleCount = 0;
	}
}

bool FParticleAnimTrailEmitterInstance::HasCompleted()
{
	return !IsTrailActive() && ActiveParticles == 0;
}

/**
 *	Retrieves the dynamic data for the emitter
 */
FDynamicEmitterDataBase* FParticleAnimTrailEmitterInstance::GetDynamicData(bool bSelected, ERHIFeatureLevel::Type InFeatureLevel)
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_ParticleAnimTrailEmitterInstance_GetDynamicData);

	UParticleLODLevel* LODLevel = SpriteTemplate->GetLODLevel(0);
	if (IsDynamicDataRequired(LODLevel) == false || !bEnabled)
	{
		return NULL;
	}

	// Allocate the dynamic data
	FDynamicAnimTrailEmitterData* NewEmitterData = new FDynamicAnimTrailEmitterData(LODLevel->RequiredModule);
	{
		INC_DWORD_STAT(STAT_DynamicEmitterCount);
		INC_DWORD_STAT(STAT_DynamicAnimTrailCount);
		INC_DWORD_STAT_BY(STAT_DynamicEmitterMem, sizeof(FDynamicAnimTrailEmitterData));
	}

	NewEmitterData->bClipSourceSegement = true;//Unused in trails.
#if WITH_EDITORONLY_DATA
	NewEmitterData->bRenderGeometry = bRenderGeometry;
	NewEmitterData->bRenderParticles = bRenderSpawnPoints;
	NewEmitterData->bRenderTangents = bRenderTangents;
	NewEmitterData->bRenderTessellation = bRenderTessellation;
#else
	NewEmitterData->bRenderGeometry = true;
	NewEmitterData->bRenderParticles = false;
	NewEmitterData->bRenderTangents = false;
	NewEmitterData->bRenderTessellation = false;
#endif	
	
	//These are unused for anim trails.
	//NewEmitterData->DistanceTessellationStepSize = TrailTypeData->DistanceTessellationStepSize;
	//NewEmitterData->TangentTessellationScalar = TrailTypeData->TangentTessellationScalar;
	//NewEmitterData->TextureTileDistance = TrailTypeData->TilingDistance;

	if (TrailTypeData->TilingDistance > 0.0f)
	{
		NewEmitterData->bTextureTileDistance = true;
	}
	else
	{
		NewEmitterData->bTextureTileDistance = false;
	}

	// Now fill in the source data
	if (!FillReplayData(NewEmitterData->Source))
	{
		delete NewEmitterData;
		return NULL;
	}

	// Setup dynamic render data.  Only call this AFTER filling in source data for the emitter.
	NewEmitterData->Init(bSelected);

	return NewEmitterData;
}

/**
 *	Retrieves replay data for the emitter
 *
 *	@return	The replay data, or NULL on failure
 */
FDynamicEmitterReplayDataBase* FParticleAnimTrailEmitterInstance::GetReplayData()
{
	if (ActiveParticles <= 0 || !bEnabled)
	{
		return NULL;
	}

	FDynamicTrailsEmitterReplayData* NewEmitterReplayData = new FDynamicTrailsEmitterReplayData();
	check(NewEmitterReplayData != NULL);

	if (!FillReplayData(*NewEmitterReplayData))
	{
		delete NewEmitterReplayData;
		return NULL;
	}

	return NewEmitterReplayData;
}

/**
 *	Retrieve the allocated size of this instance.
 *
 *	@param	OutNum			The size of this instance
 *	@param	OutMax			The maximum size of this instance
 */
void FParticleAnimTrailEmitterInstance::GetAllocatedSize(int32& OutNum, int32& OutMax)
{
	int32 Size = sizeof(FParticleAnimTrailEmitterInstance);
	int32 ActiveParticleDataSize = (ParticleData != NULL) ? (ActiveParticles * ParticleStride) : 0;
	int32 MaxActiveParticleDataSize = (ParticleData != NULL) ? (MaxActiveParticles * ParticleStride) : 0;
	int32 ActiveParticleIndexSize = (ParticleIndices != NULL) ? (ActiveParticles * sizeof(uint16)) : 0;
	int32 MaxActiveParticleIndexSize = (ParticleIndices != NULL) ? (MaxActiveParticles * sizeof(uint16)) : 0;

	OutNum = ActiveParticleDataSize + ActiveParticleIndexSize + Size;
	OutMax = MaxActiveParticleDataSize + MaxActiveParticleIndexSize + Size;
}

/**
 * Returns the size of the object/ resource for display to artists/ LDs in the Editor.
 *
 * @param	Mode	Specifies which resource size should be displayed. ( see EResourceSizeMode::Type )
 * @return  Size of resource as to be displayed to artists/ LDs in the Editor.
 */
void FParticleAnimTrailEmitterInstance::GetResourceSizeEx(FResourceSizeEx& CumulativeResourceSize)
{
	if (CumulativeResourceSize.GetResourceSizeMode() == EResourceSizeMode::EstimatedTotal || (Component.GetSceneProxy()))
	{
		int32 MaxActiveParticleDataSize = (ParticleData != NULL) ? (MaxActiveParticles * ParticleStride) : 0;
		int32 MaxActiveParticleIndexSize = (ParticleIndices != NULL) ? (MaxActiveParticles * sizeof(uint16)) : 0;
		// Take dynamic data into account as well
		CumulativeResourceSize.AddUnknownMemoryBytes(sizeof(FParticleAnimTrailEmitterInstance));
		CumulativeResourceSize.AddUnknownMemoryBytes(MaxActiveParticleDataSize);								// Copy of the particle data on the render thread
		CumulativeResourceSize.AddUnknownMemoryBytes(MaxActiveParticleIndexSize);								// Copy of the particle indices on the render thread
		if (DynamicParameterDataOffset == 0)
		{
			CumulativeResourceSize.AddUnknownMemoryBytes(MaxActiveParticles * sizeof(FParticleBeamTrailVertex));	// The vertex data array
		}
		else
		{
			CumulativeResourceSize.AddUnknownMemoryBytes(MaxActiveParticles * sizeof(FParticleBeamTrailVertexDynamicParameter));
		}
	}
}

void FParticleAnimTrailEmitterInstance::BeginTrail()
{
	//Mark any existing trails as dead.
	for (int32 FindTrailIdx = 0; FindTrailIdx < ActiveParticles; FindTrailIdx++)
	{
		int32 CheckStartIndex = ParticleIndices[FindTrailIdx];
		DECLARE_PARTICLE_PTR(CheckParticle, ParticleData + ParticleStride * CheckStartIndex);
		FAnimTrailTypeDataPayload* CheckTrailData = ((FAnimTrailTypeDataPayload*)((uint8*)CheckParticle + TypeDataOffset));
		if (CheckTrailData->TrailIndex == 0)
		{
			if (TRAIL_EMITTER_IS_START(CheckTrailData->Flags))
			{
				CheckTrailData->Flags = TRAIL_EMITTER_SET_DEADTRAIL(CheckTrailData->Flags);
				SetDeadIndex(CheckTrailData->TrailIndex, CheckStartIndex);
			}
		}
	}
	bTagTrailAsDead = false;
	bTrailEnabled = true;
}

void FParticleAnimTrailEmitterInstance::EndTrail()
{
	FirstSocketName = NAME_None;
	SecondSocketName = NAME_None;
	bTagTrailAsDead = true;
	bTrailEnabled = false;
}

void FParticleAnimTrailEmitterInstance::SetTrailSourceData(FName InFirstSocketName, FName InSecondSocketName, ETrailWidthMode InWidthMode, float InWidth )
{
	check(!InFirstSocketName.IsNone());
	check(!InSecondSocketName.IsNone());

	FirstSocketName = InFirstSocketName;
	SecondSocketName = InSecondSocketName;
	Width = InWidth;
	WidthMode = InWidthMode;
}

bool FParticleAnimTrailEmitterInstance::IsTrailActive()const 
{
	return bTrailEnabled;
}

#if WITH_EDITORONLY_DATA
void FParticleAnimTrailEmitterInstance::SetTrailDebugData(bool bInRenderGeometry, bool bInRenderSpawnPoints, bool bInRenderTessellation, bool bInRenderTangents)
{
	bRenderGeometry = bInRenderGeometry;
	bRenderSpawnPoints = bInRenderSpawnPoints;
	bRenderTessellation = bInRenderTessellation;
	bRenderTangents = bInRenderTangents;
}
#endif

/**
 * Captures dynamic replay data for this particle system.
 *
 * @param	OutData		[Out] Data will be copied here
 *
 * @return Returns true if successful
 */
bool FParticleAnimTrailEmitterInstance::FillReplayData( FDynamicEmitterReplayDataBase& OutData )
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_ParticleAnimTrailEmitterInstance_FillReplayData);

	if (ActiveParticles <= 0 || !bEnabled)
	{
		return false;
	}

	// If the template is disabled, don't return data.
	UParticleLODLevel* LODLevel = SpriteTemplate->GetLODLevel(0);
	if ((LODLevel == NULL) || (LODLevel->bEnabled == false))
	{
		return false;
	}

	// This function can modify the ParticleData (changes TriangleCount of trail payloads), so we
	// we need to call it before calling the parent implementation of FillReplayData, since that
	// will memcpy the particle data to the render thread's buffer.
	DetermineVertexAndTriangleCount();

	const int32 IndexCount = TriangleCount + 2;
	if (IndexCount > MAX_TRAIL_INDICES)
	{
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
		UE_LOGF(LogParticles, Warning, "ANIMTRAIL: FillReplayData failed.");
		UE_LOGF(LogParticles, Warning, "\tIndexCount (%d) exceeds allowed value (%d).", IndexCount, MAX_TRAIL_INDICES);
		UE_LOGF(LogParticles, Warning, "\tActiveParticleCount = %d.", ActiveParticles);
		UE_LOGF(LogParticles, Warning, "\tTriangleCount = %d.", TriangleCount);
		UE_LOGF(LogParticles, Warning, "\tTrailCount = %d.", TrailCount);
		UE_LOGF(LogParticles, Warning, "\t%ls", Component.GetTemplate() ? 
			*(Component.GetTemplate()->GetPathName()) : *(Component.GetName()));
#endif
		return false;
	}

	// Call parent implementation first to fill in common particle source data
	if (!FParticleEmitterInstance::FillReplayData(OutData))
	{
		return false;
	}

	if (TriangleCount <= 0)
	{
		if (ActiveParticles > 0 && ActiveParticles != HeadOnlyParticles)
		{
// 				UE_LOGF(LogParticles, Warning, "TRAIL: GetDynamicData -- TriangleCount == 0 (APC = %4d) for PSys %ls",
// 					ActiveParticles, 
// 					Component ? (Component->Template ? *Component->Template->GetName() : 
// 					TEXT("No Template")) : TEXT("No Component"));
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
			if (Component.HasWorld())
			{
				FString ErrorMessage = 
					FString::Printf(TEXT("ANIMTRAIL: GetDynamicData -- TriangleCount == 0 (APC = %4d) for PSys %s"),
					ActiveParticles, 
					Component.GetTemplate() ? *Component.GetTemplate()->GetName() : TEXT("No Template"));
				UE_LOGF(LogParticles, Log, "%ls", *ErrorMessage);
			}

			PrintAllActiveParticles();

#endif	//#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
		}
		return false;
	}

	OutData.eEmitterType = DET_AnimTrail;
	FDynamicTrailsEmitterReplayData* NewReplayData = static_cast<FDynamicTrailsEmitterReplayData*>( &OutData );

	NewReplayData->MaterialInterface = GetCurrentMaterial();
	// We never want local space for trails
	NewReplayData->bUseLocalSpace = false;
	// Never use axis lock for trails
	NewReplayData->bLockAxis = false;
	
	NewReplayData->MaxActiveParticleCount = MaxActiveParticles;
	NewReplayData->Sheets = 1;//Always only 1; //TrailTypeData->SheetsPerTrail ? TrailTypeData->SheetsPerTrail : 1;

	NewReplayData->VertexCount = VertexCount;
	NewReplayData->IndexCount = TriangleCount + (2 * TrailCount);
	NewReplayData->PrimitiveCount = TriangleCount;
	NewReplayData->TrailCount = TrailCount;

	//@todo.SAS. Check for requiring uint32 sized indices?
	NewReplayData->IndexStride = sizeof(uint16);

	NewReplayData->TrailDataOffset = TypeDataOffset;

	return true;
}

void FParticleAnimTrailEmitterInstance::PrintParticleData(FBaseParticle* Particle, FTrailsBaseTypeDataPayload* TrailData, int32 CurrentIndex, int32 TrailIndex)
{
	UE_LOGF(LogParticles, Log, "%d| Particle %4d - Next = %4d, Prev = %4d, Type = %8ls, Life=%.5f",
		TrailIndex,
		CurrentIndex,
		TRAIL_EMITTER_GET_NEXT(TrailData->Flags),
		TRAIL_EMITTER_GET_PREV(TrailData->Flags),
		TRAIL_EMITTER_IS_ONLY(TrailData->Flags) ? TEXT("ONLY") :
		TRAIL_EMITTER_IS_START(TrailData->Flags) ? TEXT("START") :
		TRAIL_EMITTER_IS_END(TrailData->Flags) ? TEXT("END") :
		TRAIL_EMITTER_IS_MIDDLE(TrailData->Flags) ? TEXT("MIDDLE") :
		TRAIL_EMITTER_IS_DEADTRAIL(TrailData->Flags) ? TEXT("DEAD") :
		TEXT("????"),
		Particle->RelativeTime
		);
	UE_LOGF(LogParticles, Log, "TI=%4d - TC=%4d, SpT=%.5f, SpD= %.5f, TU=%.4f, SpTP=%4d, RIntpC=%4d, PSF=%.4f, %ls, %ls",
		TrailData->TrailIndex, TrailData->TriangleCount, TrailData->SpawnTime, TrailData->SpawnDelta, TrailData->TiledU, TrailData->SpawnedTessellationPoints,
		TrailData->RenderingInterpCount, TrailData->PinchScaleFactor, TrailData->bInterpolatedSpawn ? TEXT("1") : TEXT("0"), TrailData->bMovementSpawned ? TEXT("1") : TEXT("0"));
}

void FParticleAnimTrailEmitterInstance::PrintAllActiveParticles()
{
	UE_LOGF(LogParticles, Log, "===========================================================================");
	//Print out the state of all the particles. 
	for (int32 ActiveIdx = 0; ActiveIdx < ActiveParticles; ++ActiveIdx)
	{
		int32 CurrentIndex = ParticleIndices[ActiveIdx];
		DECLARE_PARTICLE_PTR(Particle, ParticleData + ParticleStride * CurrentIndex);
		FTrailsBaseTypeDataPayload* TrailData = ((FTrailsBaseTypeDataPayload*)((uint8*)Particle + TypeDataOffset));
		PrintParticleData(Particle, TrailData, CurrentIndex, -1);
		UE_LOGF(LogParticles, Log, "---------------------------------------------------------------------------------------");
	}
	UE_LOGF(LogParticles, Log, "===========================================================================");
}

void FParticleAnimTrailEmitterInstance::PrintTrails()
{
	if (ActiveParticles > 0)
	{
		UE_LOGF(LogParticles, Log, "===========================================");
		UE_LOGF(LogParticles, Log, "Active: %d", ActiveParticles);
		UE_LOGF(LogParticles, Log, "===========================================");
		TArray<int32> ParticlesVisited;
		TArray<int32> TrailHeads;
		for (int32 ActiveIdx = 0; ActiveIdx < ActiveParticles; ++ActiveIdx)
		{
			int32 CurrentIndex = ParticleIndices[ActiveIdx];
			//Work forwards through the trail printing data about the trail.
			DECLARE_PARTICLE_PTR(Particle, ParticleData + ParticleStride * CurrentIndex);
			FTrailsBaseTypeDataPayload* TrailData = ((FTrailsBaseTypeDataPayload*)((uint8*)Particle + TypeDataOffset));
			if (TRAIL_EMITTER_IS_HEAD(TrailData->Flags))
			{
				TrailHeads.Add(CurrentIndex);
				while (true)
				{
					//Ensure we've not already visited this particle.
					check(!ParticlesVisited.Contains(CurrentIndex));

					ParticlesVisited.Add(CurrentIndex);

					PrintParticleData(Particle, TrailData, CurrentIndex, TrailHeads.Num() - 1);

					int32 Next = TRAIL_EMITTER_GET_NEXT(TrailData->Flags);
					if (Next == TRAIL_EMITTER_NULL_NEXT)
					{
						UE_LOGF(LogParticles, Log, "===========================================================================");
						break;
					}
					else
					{
						UE_LOGF(LogParticles, Log, "---------------------------------------------------------------------------------------");
						CurrentIndex = Next;
						Particle = (FBaseParticle*)(ParticleData + ParticleStride * Next);
						TrailData = ((FTrailsBaseTypeDataPayload*)((uint8*)Particle + TypeDataOffset));
					}
				}
			}
		}

		//check that all particles were visited. If not then there are some orphaned particles munging things up.
		if (ParticlesVisited.Num() != ActiveParticles)
		{
			PrintAllActiveParticles();
		}
	}
}

static int32 GbEnableCircularAnimTrailDump = 2;
static FAutoConsoleVariableRef CVarEnableCircularAnimTrailDump(
	TEXT("fx.EnableCircularAnimTrailDump"),
	GbEnableCircularAnimTrailDump,
	TEXT("Controls logging for when circular links are discovered in anim trails.\n0 = No logging.\n1 = Minimal logging.\n2 = Verbose logging."),
	ECVF_Default
);

void FParticleTrailsEmitterInstance_Base::DumpCircularTrailsSpam()
{
	if (GbEnableCircularAnimTrailDump > 0)
	{
		UE_LOGF(LogParticles, Warning, "Circular links in Anim Trail discovered. \nPSys: %ls\nEmitter: %ls", *Component.GetTemplate()->GetFullName(), *SpriteTemplate->GetEmitterName().ToString());
		if (GbEnableCircularAnimTrailDump > 1)
		{
			PrintAllActiveParticles();
		}
	}
}
