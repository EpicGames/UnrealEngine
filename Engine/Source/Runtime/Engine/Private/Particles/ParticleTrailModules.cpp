// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	ParticleTrailModules.cpp: Particle module implementations for trails.
=============================================================================*/

#include "ParticleEmitterInstances.h"
#include "Particles/ParticleModule.h"
#include "Particles/ParticleSystemComponent.h"
#include "Distributions/DistributionFloatConstant.h"
#include "Particles/Trail/ParticleModuleTrailBase.h"
#include "Particles/Trail/ParticleModuleTrailSource.h"
#include "Particles/TypeData/ParticleModuleTypeDataAnimTrail.h"
#include "Particles/TypeData/ParticleModuleTypeDataBase.h"
#include "Particles/TypeData/ParticleModuleTypeDataRibbon.h"
#include "ParticleBeamTrailVertexFactory.h"
#include "UObject/UnrealType.h"
#include "Particles/ParticleEmitter.h"

UParticleModuleTrailBase::UParticleModuleTrailBase(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bSpawnModule = false;
	bUpdateModule = false;
}
/*-----------------------------------------------------------------------------
	Abstract base modules used for categorization.
-----------------------------------------------------------------------------*/


/*-----------------------------------------------------------------------------
	UParticleModuleTrailSource implementation.
-----------------------------------------------------------------------------*/

UParticleModuleTrailSource::UParticleModuleTrailSource(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	// Structure to hold one-time initialization
	struct FConstructorStatics
	{
		FName NAME_None;
		FConstructorStatics()
			: NAME_None(TEXT("None"))
		{
		}
	};
	static FConstructorStatics ConstructorStatics;

	SourceMethod = PET2SRCM_Default;
	SourceName = ConstructorStatics.NAME_None;
	SelectionMethod = EPSSM_Sequential;
	bInheritRotation = false;
}

void UParticleModuleTrailSource::InitializeDefaults()
{
	if (!SourceStrength.IsCreated())
	{
		UDistributionFloatConstant* DistributionSourceStrength = NewObject<UDistributionFloatConstant>(this, TEXT("DistributionSourceStrength"));
		DistributionSourceStrength->Constant = 100.0f;
		SourceStrength.Distribution = DistributionSourceStrength;
	}
}

void UParticleModuleTrailSource::PostInitProperties()
{
	Super::PostInitProperties();
	if (!HasAnyFlags(RF_ClassDefaultObject | RF_NeedLoad))
	{
		InitializeDefaults();
	}
}

#if WITH_EDITOR
void UParticleModuleTrailSource::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	InitializeDefaults();

//	SourceOffsetCount
//	SourceOffsetDefaults
	FProperty* PropertyThatChanged = PropertyChangedEvent.Property;
	if (PropertyThatChanged)
	{
		if (PropertyThatChanged->GetFName() == FName("SourceOffsetCount"))
		{
			if (SourceOffsetDefaults.Num() > 0)
			{
				if (SourceOffsetDefaults.Num() < SourceOffsetCount)
				{
					// Add additional slots
					SourceOffsetDefaults.InsertZeroed(SourceOffsetDefaults.Num(), SourceOffsetCount - SourceOffsetDefaults.Num());
				}
				else
				if (SourceOffsetDefaults.Num() > SourceOffsetCount)
				{
					// Remove the required slots
					int32	RemoveIndex	= SourceOffsetCount ? (SourceOffsetCount - 1) : 0;
					SourceOffsetDefaults.RemoveAt(RemoveIndex, SourceOffsetDefaults.Num() - SourceOffsetCount);
				}
			}
			else
			{
				if (SourceOffsetCount > 0)
				{
					// Add additional slots
					SourceOffsetDefaults.InsertZeroed(0, SourceOffsetCount);
				}
			}
		}
	}

	Super::PostEditChangeProperty(PropertyChangedEvent);
}
#endif // WITH_EDITOR

void UParticleModuleTrailSource::AutoPopulateInstanceProperties(UParticleSystemComponent* PSysComp)
{
	check(IsInGameThread());
	switch (SourceMethod)
	{
	case PET2SRCM_Actor:
		{
			bool	bFound	= false;

			for (int32 i = 0; i < PSysComp->InstanceParameters.Num(); i++)
			{
				FParticleSysParam* Param = &(PSysComp->InstanceParameters[i]);
				
				if (Param->Name == SourceName)
				{
					bFound	=	true;
					break;
				}
			}

			if (!bFound)
			{
				int32 NewParamIndex = PSysComp->InstanceParameters.AddZeroed();
				PSysComp->InstanceParameters[NewParamIndex].Name		= SourceName;
				PSysComp->InstanceParameters[NewParamIndex].ParamType	= PSPT_Actor;
				PSysComp->InstanceParameters[NewParamIndex].Actor		= NULL;
			}
		}
		break;
	}
}


void UParticleModuleTrailSource::GetParticleSysParamsUtilized(TArray<FString>& ParticleSysParamList)
{
	if (SourceMethod == PET2SRCM_Actor)
	{
		ParticleSysParamList.Add(FString::Printf(TEXT("TrailSource: Actor: %s\n"), *(SourceName.ToString())));
	}
}


bool UParticleModuleTrailSource::ResolveSourceOffset(int32 InTrailIdx, FParticleEmitterInstance* InEmitterInst, FVector& OutSourceOffset)
{
	// For now, we are only supporting the default values (for ribbon emitters)
	if (InTrailIdx < SourceOffsetDefaults.Num())
	{
		OutSourceOffset = SourceOffsetDefaults[InTrailIdx];
		return true;
	}
	return false;
}

/*-----------------------------------------------------------------------------
	UParticleModuleTypeDataRibbon implementation.
-----------------------------------------------------------------------------*/
UParticleModuleTypeDataRibbon::UParticleModuleTypeDataRibbon(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	MaxTessellationBetweenParticles = 25;
	SheetsPerTrail = 1;
	MaxTrailCount = 1;
	MaxParticleInTrailCount = 500;
	bDeadTrailsOnDeactivate = true;
	bClipSourceSegement = true;
	bEnablePreviousTangentRecalculation = true;
	bTangentRecalculationEveryFrame = false;
	bDeadTrailsOnSourceLoss = true;
	TangentSpawningScalar = 0.0f;
	bRenderGeometry = true;
	bRenderSpawnPoints = false;
	bRenderTangents = false;
	bRenderTessellation = false;
	DistanceTessellationStepSize = 15.0f;
	TangentTessellationScalar = 5.0f;
}

uint32 UParticleModuleTypeDataRibbon::RequiredBytes(UParticleModuleTypeDataBase* TypeData)
{
	return sizeof(FRibbonTypeDataPayload);
}

#if WITH_EDITOR
void UParticleModuleTypeDataRibbon::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	FProperty* PropertyThatChanged = PropertyChangedEvent.Property;
	if (PropertyThatChanged && PropertyThatChanged->GetName() == TEXT("MaxTessellationBetweenParticles"))
	{
		if (MaxTessellationBetweenParticles < 0)
		{
			MaxTessellationBetweenParticles = 0;
		}
	}
	else if (PropertyThatChanged && PropertyThatChanged->GetName() == TEXT("SheetsPerTrail"))
	{
		if (SheetsPerTrail <= 0)
		{
			SheetsPerTrail = 1;
		}
	}
	else if (PropertyThatChanged && PropertyThatChanged->GetName() == TEXT("MaxTrailCount"))
	{
		if (MaxTrailCount <= 0)
		{
			MaxTrailCount = 1;
		}
	}
	else if (PropertyThatChanged && PropertyThatChanged->GetName() == TEXT("MaxParticleInTrailCount"))
	{
		if (MaxParticleInTrailCount < 0)
		{
			MaxParticleInTrailCount = 0;
		}
	}
}
#endif // WITH_EDITOR

FParticleEmitterInstance* UParticleModuleTypeDataRibbon::CreateInstance(UParticleEmitter* InEmitterParent, IParticleEmitterInstanceOwner& InComponent)
{
	FParticleEmitterInstance* Instance = new FParticleRibbonEmitterInstance(InComponent);
	check(Instance);
	Instance->InitParameters(InEmitterParent);
	return Instance;
}

const FVertexFactoryType* UParticleModuleTypeDataRibbon::GetVertexFactoryType() const
{
	return &FParticleBeamTrailVertexFactory::StaticType;
}

void UParticleModuleTypeDataRibbon::CollectPSOPrecacheData(const UParticleEmitter* Emitter, FPSOPrecacheParams& OutParams)
{
	bool bUsesDynamicParameter = (Emitter->DynamicParameterDataOffset > 0);

	FPSOPrecacheVertexFactoryData VFData;
	VFData.VertexFactoryType = &FParticleBeamTrailVertexFactory::StaticType;
	VFData.CustomDefaultVertexDeclaration = FParticleBeamTrailVertexFactory::GetPSOPrecacheVertexDeclaration(bUsesDynamicParameter);
	OutParams.VertexFactoryDataList.Add(VFData);
	OutParams.PrimitiveType = GetPrimitiveType();
}

/*-----------------------------------------------------------------------------
	UParticleModuleTypeDataAnimTrail implementation.
-----------------------------------------------------------------------------*/
UParticleModuleTypeDataAnimTrail::UParticleModuleTypeDataAnimTrail(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bDeadTrailsOnDeactivate = true;
	bEnablePreviousTangentRecalculation = true;
	bTangentRecalculationEveryFrame = false;
	DistanceTessellationStepSize = 10.0f;
	TangentTessellationStepSize = 0.0f;
}

uint32 UParticleModuleTypeDataAnimTrail::RequiredBytes(UParticleModuleTypeDataBase* TypeData)
{
	return sizeof(FAnimTrailTypeDataPayload);
}

FParticleEmitterInstance* UParticleModuleTypeDataAnimTrail::CreateInstance(UParticleEmitter* InEmitterParent, IParticleEmitterInstanceOwner& InComponent)
{
	FParticleEmitterInstance* Instance = new FParticleAnimTrailEmitterInstance(InComponent);
	check(Instance);
	Instance->InitParameters(InEmitterParent);
	return Instance;
}

const FVertexFactoryType* UParticleModuleTypeDataAnimTrail::GetVertexFactoryType() const
{
	return &FParticleBeamTrailVertexFactory::StaticType;
}

void UParticleModuleTypeDataAnimTrail::CollectPSOPrecacheData(const UParticleEmitter* Emitter, FPSOPrecacheParams& OutParams)
{
	bool bUsesDynamicParameter = (Emitter->DynamicParameterDataOffset > 0);

	FPSOPrecacheVertexFactoryData VFData;
	VFData.VertexFactoryType = &FParticleBeamTrailVertexFactory::StaticType;
	VFData.CustomDefaultVertexDeclaration = FParticleBeamTrailVertexFactory::GetPSOPrecacheVertexDeclaration(bUsesDynamicParameter);
	OutParams.VertexFactoryDataList.Add(VFData);
	OutParams.PrimitiveType = GetPrimitiveType();
}
