// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	ParticleVertexFactory.cpp: Particle vertex factory implementation.
=============================================================================*/

#include "ParticleVertexFactory.h"
#include "MeshDrawShaderBindings.h"
#include "ParticleHelper.h"
#include "Misc/DelayedAutoRegister.h"
#include "ParticleResources.h"
#include "MeshMaterialShader.h"
#include "PipelineStateCache.h"

IMPLEMENT_GLOBAL_SHADER_PARAMETER_STRUCT(FParticleSpriteUniformParameters, "SpriteVF");

TGlobalResource<FNullDynamicParameterVertexBuffer> GNullDynamicParameterVertexBuffer;

class FNullSubUVCutoutVertexBuffer : public FVertexBuffer
{
public:
	/**
	 * Initialize the RHI for this rendering resource
	 */
	virtual void InitRHI(FRHICommandListBase& RHICmdList) override
	{
		// create a static vertex buffer
		const FRHIBufferCreateDesc CreateDesc =
			FRHIBufferCreateDesc::CreateVertex<FVector2f>(TEXT("FNullSubUVCutoutVertexBuffer"), 4)
			.AddUsage(EBufferUsageFlags::Static | EBufferUsageFlags::ShaderResource)
			.SetInitialState(ERHIAccess::VertexOrIndexBuffer | ERHIAccess::SRVMask)
			.SetInitActionZeroData();

		VertexBufferRHI = RHICmdList.CreateBuffer(CreateDesc);
		VertexBufferSRV = RHICmdList.CreateShaderResourceView(
			VertexBufferRHI,
			FRHIViewDesc::CreateBufferSRV()
				.SetType(FRHIViewDesc::EBufferType::Typed)
				.SetFormat(PF_G32R32F));
	}
	
	virtual void ReleaseRHI() override
	{
		VertexBufferSRV.SafeRelease();
		FVertexBuffer::ReleaseRHI();
	}
	
	FShaderResourceViewRHIRef VertexBufferSRV;
};
TGlobalResource<FNullSubUVCutoutVertexBuffer> GFNullSubUVCutoutVertexBuffer;

/**
 * Shader parameters for the particle vertex factory.
 */
class FParticleSpriteVertexFactoryShaderParameters : public FVertexFactoryShaderParameters
{
	DECLARE_TYPE_LAYOUT(FParticleSpriteVertexFactoryShaderParameters, NonVirtual);
public:
};

IMPLEMENT_TYPE_LAYOUT(FParticleSpriteVertexFactoryShaderParameters);

class FParticleSpriteVertexFactoryShaderParametersVS : public FParticleSpriteVertexFactoryShaderParameters
{
	DECLARE_TYPE_LAYOUT(FParticleSpriteVertexFactoryShaderParametersVS, NonVirtual);
public:
	void Bind(const FShaderParameterMap& ParameterMap)
	{
		NumCutoutVerticesPerFrame.Bind(ParameterMap, TEXT("NumCutoutVerticesPerFrame"));
		CutoutGeometry.Bind(ParameterMap, TEXT("CutoutGeometry"));
	}

	void GetElementShaderBindings(
		const FSceneInterface* Scene,
		const FSceneView* View,
		const FMeshMaterialShader* Shader,
		const EVertexInputStreamType InputStreamType,
		ERHIFeatureLevel::Type FeatureLevel,
		const FVertexFactory* VertexFactory,
		const FMeshBatchElement& BatchElement,
		class FMeshDrawSingleShaderBindings& ShaderBindings,
		FVertexInputStreamArray& VertexStreams) const
	{
		FParticleSpriteVertexFactory* SpriteVF = (FParticleSpriteVertexFactory*)VertexFactory;
		ShaderBindings.Add(Shader->GetUniformBufferParameter<FParticleSpriteUniformParameters>(), SpriteVF->GetSpriteUniformBuffer() );
		
		ShaderBindings.Add(NumCutoutVerticesPerFrame, SpriteVF->GetNumCutoutVerticesPerFrame());
		FRHIShaderResourceView* NullSRV = GFNullSubUVCutoutVertexBuffer.VertexBufferSRV;
		ShaderBindings.Add(CutoutGeometry, SpriteVF->GetCutoutGeometrySRV() ? SpriteVF->GetCutoutGeometrySRV() : NullSRV);
	}

private:
	LAYOUT_FIELD(FShaderParameter, NumCutoutVerticesPerFrame);
	LAYOUT_FIELD(FShaderResourceParameter, CutoutGeometry);
};

IMPLEMENT_TYPE_LAYOUT(FParticleSpriteVertexFactoryShaderParametersVS);

class FParticleSpriteVertexFactoryShaderParametersPS : public FParticleSpriteVertexFactoryShaderParameters
{
	DECLARE_TYPE_LAYOUT(FParticleSpriteVertexFactoryShaderParametersPS, NonVirtual);
public:
	void GetElementShaderBindings(
		const FSceneInterface* Scene,
		const FSceneView* View,
		const FMeshMaterialShader* Shader,
		const EVertexInputStreamType InputStreamType,
		ERHIFeatureLevel::Type FeatureLevel,
		const FVertexFactory* VertexFactory,
		const FMeshBatchElement& BatchElement,
		class FMeshDrawSingleShaderBindings& ShaderBindings,
		FVertexInputStreamArray& VertexStreams) const
	{
		FParticleSpriteVertexFactory* SpriteVF = (FParticleSpriteVertexFactory*)VertexFactory;
		ShaderBindings.Add(Shader->GetUniformBufferParameter<FParticleSpriteUniformParameters>(), SpriteVF->GetSpriteUniformBuffer() );
	}
};

IMPLEMENT_TYPE_LAYOUT(FParticleSpriteVertexFactoryShaderParametersPS);

/**
 * The particle system vertex declaration resource type.
 */
class FParticleSpriteVertexDeclaration : public FRenderResource
{
public:

	FVertexDeclarationRHIRef VertexDeclarationRHI;

	// Constructor.
	FParticleSpriteVertexDeclaration(bool bInInstanced, bool bInUsesDynamicParameter) :
		bInstanced(bInInstanced),
		bUsesDynamicParameter(bInUsesDynamicParameter)
	{

	}

	// Destructor.
	virtual ~FParticleSpriteVertexDeclaration() {}

	virtual void FillDeclElements(FVertexDeclarationElementList& Elements, int32& Offset)
	{
		uint32 InitialStride = sizeof(float) * 2;
		uint32 PerParticleStride = sizeof(FParticleSpriteVertex);

		/** The stream to read the texture coordinates from. */
		check( Offset == 0 );
		uint32 Stride = bInstanced ? InitialStride : InitialStride + PerParticleStride;
		Elements.Add(FVertexElement(0, Offset, VET_Float2, 4, Stride, false));
		Offset += sizeof(float) * 2;

		/** The per-particle streams follow. */
		if(bInstanced) 
		{
			Offset = 0;
			// update stride
			Stride = PerParticleStride;
		}

		/** The stream to read the vertex position from. */
		Elements.Add(FVertexElement(bInstanced ? 1 : 0, Offset, VET_Float4, 0, Stride, bInstanced));
		Offset += sizeof(float) * 4;
		/** The stream to read the vertex old position from. */
		Elements.Add(FVertexElement(bInstanced ? 1 : 0, Offset, VET_Float4, 1, Stride, bInstanced));
		Offset += sizeof(float) * 4;
		/** The stream to read the vertex size/rot/subimage from. */
		Elements.Add(FVertexElement(bInstanced ? 1 : 0, Offset, VET_Float4, 2, Stride, bInstanced));
		Offset += sizeof(float) * 4;
		/** The stream to read the color from.					*/
		Elements.Add(FVertexElement(bInstanced ? 1 : 0, Offset, VET_Float4, 3, Stride, bInstanced));
		Offset += sizeof(float) * 4;
		
		/** The per-particle dynamic parameter stream */

		// The -V519 disables a warning from PVS-Studio's static analyzer. It noticed that offset is assigned
		// twice before being read. It is probably safer to leave the redundant assignments here to reduce
		// the chance of an error being introduced if this code is modified.
		Offset = 0;  //-V519
		Stride = sizeof(float) * 4;
		Elements.Add(FVertexElement(bInstanced ? 2 : 1, Offset, VET_Float4, 5, bUsesDynamicParameter ? Stride : 0, bInstanced));
		Offset += sizeof(float) * 4;
	}

	virtual void InitRHI(FRHICommandListBase& RHICmdList)
	{
		FVertexDeclarationElementList Elements;
		int32	Offset = 0;

		FillDeclElements(Elements, Offset);

		// Create the vertex declaration for rendering the factory normally.
		VertexDeclarationRHI = PipelineStateCache::GetOrCreateVertexDeclaration(Elements);
	}

	virtual void ReleaseRHI()
	{
		VertexDeclarationRHI.SafeRelease();
	}

private:
	bool bInstanced;
	bool bUsesDynamicParameter;
	int32 NumVertsInInstanceBuffer;
};

/** The simple element vertex declaration. */
static TGlobalResource<FParticleSpriteVertexDeclaration> GParticleSpriteVertexDeclarationInstanced(true, false);
static TGlobalResource<FParticleSpriteVertexDeclaration> GParticleSpriteVertexDeclarationInstancedDynamic(true, true);

static inline TGlobalResource<FParticleSpriteVertexDeclaration>& GetParticleSpriteVertexDeclaration(int32 NumVertsInInstanceBuffer, bool bUsesDynamicParameter)
{
	check(NumVertsInInstanceBuffer == 4 || NumVertsInInstanceBuffer == 8);
	if (bUsesDynamicParameter)
	{
		return GParticleSpriteVertexDeclarationInstancedDynamic;
	}
	else
	{
		return GParticleSpriteVertexDeclarationInstanced;
	}
}

bool FParticleSpriteVertexFactory::ShouldCompilePermutation(const FVertexFactoryShaderPermutationParameters& Parameters)
{
	return Parameters.MaterialParameters.bIsUsedWithParticleSprites || Parameters.MaterialParameters.bIsSpecialEngineMaterial;
}

/**
 * Can be overridden by FVertexFactory subclasses to modify their compile environment just before compilation occurs.
 */
void FParticleSpriteVertexFactory::ModifyCompilationEnvironment(const FVertexFactoryShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
{
	FParticleVertexFactoryBase::ModifyCompilationEnvironment(Parameters, OutEnvironment);

	// Set a define so we can tell in MaterialTemplate.usf when we are compiling a sprite vertex factory
	OutEnvironment.SetDefine(TEXT("PARTICLE_SPRITE_FACTORY"),TEXT("1"));
}

/**
 * Get vertex elements used when during PSO precaching materials using this vertex factory type
 */
void FParticleSpriteVertexFactory::GetPSOPrecacheVertexFetchElements(EVertexInputStreamType VertexInputStreamType, FVertexDeclarationElementList& Elements)
{
	GParticleSpriteVertexDeclarationInstanced.VertexDeclarationRHI->GetInitializer(Elements);
}

FRHIVertexDeclaration* FParticleSpriteVertexFactory::GetPSOPrecacheVertexDeclaration(bool bUsesDynamicParameter)
{
	return GetParticleSpriteVertexDeclaration(4, bUsesDynamicParameter).VertexDeclarationRHI;
}

/**
 *	Initialize the Render Hardware Interface for this vertex factory
 */
void FParticleSpriteVertexFactory::InitRHI(FRHICommandListBase& RHICmdList)
{
	InitStreams();
	SetDeclaration(GetParticleSpriteVertexDeclaration(NumVertsInInstanceBuffer, bUsesDynamicParameter).VertexDeclarationRHI);
}

void FParticleSpriteVertexFactory::InitStreams()
{
	check(Streams.Num() == 0);
	FVertexStream* TexCoordStream = new(Streams) FVertexStream;
	TexCoordStream->VertexBuffer = &GParticleTexCoordVertexBuffer;
	TexCoordStream->Stride = sizeof(FVector2f);
	TexCoordStream->Offset = 0;
	FVertexStream* InstanceStream = new(Streams) FVertexStream;
	FVertexStream* DynamicParameterStream = new(Streams) FVertexStream;
	DynamicParameterStream->Stride = bUsesDynamicParameter ? DynamicParameterStride : 0;
}

void FParticleSpriteVertexFactory::SetInstanceBuffer(const FVertexBuffer* InInstanceBuffer, uint32 StreamOffset, uint32 Stride)
{
	check(Streams.Num() == 3);
	FVertexStream& InstanceStream = Streams[1];
	InstanceStream.VertexBuffer = InInstanceBuffer;
	InstanceStream.Stride = Stride;
	InstanceStream.Offset = StreamOffset;
}

void FParticleSpriteVertexFactory::SetTexCoordBuffer(const FVertexBuffer* InTexCoordBuffer)
{
	FVertexStream& TexCoordStream = Streams[0];
	TexCoordStream.VertexBuffer = InTexCoordBuffer;
}

void FParticleSpriteVertexFactory::SetDynamicParameterBuffer(const FVertexBuffer* InDynamicParameterBuffer, uint32 StreamOffset, uint32 Stride)
{
	check(Streams.Num() == 3);
	FVertexStream& DynamicParameterStream = Streams[2];
	if (InDynamicParameterBuffer)
	{
		ensure(bUsesDynamicParameter);
		DynamicParameterStream.VertexBuffer = InDynamicParameterBuffer;
		ensure(DynamicParameterStream.Stride == Stride);
		DynamicParameterStream.Offset = StreamOffset;
	}
	else
	{
		ensure(!bUsesDynamicParameter);
		DynamicParameterStream.VertexBuffer = &GNullDynamicParameterVertexBuffer;
		ensure(DynamicParameterStream.Stride == 0);
		DynamicParameterStream.Offset = 0;
	}
}

IMPLEMENT_VERTEX_FACTORY_PARAMETER_TYPE(FParticleSpriteVertexFactory, SF_Vertex, FParticleSpriteVertexFactoryShaderParametersVS);
IMPLEMENT_VERTEX_FACTORY_PARAMETER_TYPE(FParticleSpriteVertexFactory, SF_Pixel, FParticleSpriteVertexFactoryShaderParametersPS);
IMPLEMENT_VERTEX_FACTORY_TYPE(FParticleSpriteVertexFactory,"/Engine/Private/ParticleSpriteVertexFactory.ush",
	  EVertexFactoryFlags::UsedWithMaterials
	| EVertexFactoryFlags::SupportsDynamicLighting
	| EVertexFactoryFlags::SupportsPSOPrecaching
);
