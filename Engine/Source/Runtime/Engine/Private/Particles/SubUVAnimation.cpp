// Copyright Epic Games, Inc. All Rights Reserved.

#include "Particles/SubUVAnimation.h"
#include "Containers/ClosableMpscQueue.h"
#include "Engine/Texture2D.h"
#include "Experimental/Containers/HazardPointer.h"
#include "Math/RandomStream.h"
#include "Math/ConvexHull2d.h"
#include "ParticleHelper.h"
#include "Particles/ParticleSystemComponent.h"
#include "DerivedDataCacheInterface.h"
#include "DerivedDataCacheKey.h"
#include "ComponentReregisterContext.h"
#include "RHI.h"
#include "RHIResourceUtils.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SubUVAnimation)

#define LOCTEXT_NAMESPACE "FSubUVDerivedData"

#if ENABLE_COOK_STATS
FCookStats::FDDCResourceUsageStats SubUVAnimationCookStats::UsageStats;
FCookStatsManager::FAutoRegisterCallback SubUVAnimationCookStats::RegisterCookStats([](FCookStatsManager::AddStatFuncRef AddStat)
{
	UsageStats.LogStats(AddStat, TEXT("SubUVAnimation.Usage"), TEXT(""));
});
#endif

FString FSubUVDerivedData::GetDDCKeyString(const FGuid& StateId, int32 SizeX, int32 SizeY, int32 Mode, float AlphaThreshold, int32 OpacitySourceMode)
{
	FString KeyString = FString::Printf(TEXT("%s_%u_%u_%u_%f"), *StateId.ToString(), SizeX, SizeY, Mode, AlphaThreshold);

	if (OpacitySourceMode != 0)
	{
		KeyString += FString::Printf(TEXT("_%u"), OpacitySourceMode);
	}
	// adding v2 to the key after fixing color channel offsets
	// adding v3 to the key after allowing other formats
	// adding v4 to the key after adding G8 support
	KeyString += TEXT("_V4");

#if WITH_EDITOR
	static UE::DerivedData::FCacheBucket LegacyBucket(TEXTVIEW("LegacySUBUV"), TEXTVIEW("SubUV"));
#endif

	return FDerivedDataCacheInterface::BuildCacheKey(TEXT("SUBUV_"), SUBUV_DERIVEDDATA_VER, *KeyString);
}

void FSubUVDerivedData::Serialize(FStructuredArchive::FSlot Slot)
{
	Slot << BoundingGeometry;
}

FSubUVBoundingGeometryBuffer::FSubUVBoundingGeometryBuffer() = default;

FSubUVBoundingGeometryBuffer::FSubUVBoundingGeometryBuffer(TArray<FVector2f>* InVertices)
{
	Vertices = InVertices;
}

FSubUVBoundingGeometryBuffer::~FSubUVBoundingGeometryBuffer() = default;

void FSubUVBoundingGeometryBuffer::InitRHI(FRHICommandListBase& RHICmdList)
{
	if (Vertices->Num() > 0)
	{
		VertexBufferRHI = UE::RHIResourceUtils::CreateVertexBufferFromArray(RHICmdList, TEXT("FSubUVBoundingGeometryBuffer"), EBufferUsageFlags::ShaderResource | EBufferUsageFlags::Static, MakeConstArrayView(*Vertices));
		ShaderResourceView = RHICmdList.CreateShaderResourceView(
			VertexBufferRHI,
			FRHIViewDesc::CreateBufferSRV()
				.SetType(FRHIViewDesc::EBufferType::Typed)
				.SetFormat(PF_G32R32F));
	}
}

void FSubUVBoundingGeometryBuffer::ReleaseRHI()
{
	FVertexBuffer::ReleaseRHI();
	ShaderResourceView.SafeRelease();
}

USubUVAnimation::USubUVAnimation(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	SubImages_Horizontal = 8;
	SubImages_Vertical = 8;
	BoundingMode = BVC_EightVertices;
}

void USubUVAnimation::PostInitProperties()
{
	Super::PostInitProperties();

	if (!HasAnyFlags(RF_ClassDefaultObject))
	{
		BoundingGeometryBuffer = new FSubUVBoundingGeometryBuffer(&DerivedData.BoundingGeometry);
	}
}

void USubUVAnimation::Serialize(FStructuredArchive::FRecord Record)
{
	FArchive& UnderlyingArchive = Record.GetUnderlyingArchive();

	Super::Serialize(Record);

	bool bCooked = UnderlyingArchive.IsCooking();

	// Save a bool indicating whether this is cooked data
	// This is needed when loading cooked data, to know to serialize differently
	Record << SA_VALUE(TEXT("bCooked"), bCooked);

	if (FPlatformProperties::RequiresCookedData() && !bCooked && UnderlyingArchive.IsLoading())
	{
		UE_LOGF(LogParticles, Fatal, "This platform requires cooked packages, and this SubUV animation does not contain cooked data %ls.", *GetName());
	}

	if (bCooked)
	{
		DerivedData.Serialize(Record.EnterField(TEXT("DerivedData")));
	}
}

void USubUVAnimation::CacheDerivedData()
{
#if WITH_EDITORONLY_DATA

	if (!SubUVTexture)
	{
		UE_LOGF(LogParticles, Warning, "SubUVAnimation %ls set with a NULL texture, particle geometry will be a quad by default.", *GetName());
	}
	
	FGuid SubUVGuid = SubUVTexture ? SubUVTexture->Source.GetId() : FGuid(0, 0, 0, 0);
	const FString KeyString = FSubUVDerivedData::GetDDCKeyString(SubUVGuid, SubImages_Horizontal, SubImages_Vertical, (int32)BoundingMode, AlphaThreshold, (int32)OpacitySourceMode);
	TArray<uint8> Data;

	COOK_STAT(auto Timer = SubUVAnimationCookStats::UsageStats.TimeSyncWork());
	if (GetDerivedDataCacheRef().GetSynchronous(*KeyString, Data, GetPathName()))
	{
		COOK_STAT(Timer.AddHit(Data.Num()));
		DerivedData.BoundingGeometry.Empty(Data.Num() / sizeof(FVector2f));
		DerivedData.BoundingGeometry.AddUninitialized(Data.Num() / sizeof(FVector2f));
		FPlatformMemory::Memcpy(DerivedData.BoundingGeometry.GetData(), Data.GetData(), Data.Num() * Data.GetTypeSize());
	}
	else
	{
		DerivedData.Build(SubUVTexture, SubImages_Horizontal, SubImages_Vertical, BoundingMode, AlphaThreshold, OpacitySourceMode);

		Data.Empty(DerivedData.BoundingGeometry.Num() * sizeof(FVector2f));
		Data.AddUninitialized(DerivedData.BoundingGeometry.Num() * sizeof(FVector2f));
		FPlatformMemory::Memcpy(Data.GetData(), DerivedData.BoundingGeometry.GetData(), DerivedData.BoundingGeometry.Num() * DerivedData.BoundingGeometry.GetTypeSize());
		GetDerivedDataCacheRef().Put(*KeyString, Data, GetPathName());
		COOK_STAT(Timer.AddMiss(Data.Num()));
	}
#endif
}

void USubUVAnimation::PostLoad()
{
	Super::PostLoad();
	
	if (!FPlatformProperties::RequiresCookedData())
	{
		if (SubUVTexture)
		{
			SubUVTexture->ConditionalPostLoad();
		}

		CacheDerivedData();
	}

	// The SRV is only needed for platforms that can render particles with instancing
	BeginInitResource(BoundingGeometryBuffer);
}

#if WITH_EDITOR

void USubUVAnimation::PreEditChange(FProperty* PropertyThatChanged)
{
	Super::PreEditChange(PropertyThatChanged);

	// Particle rendering is reading from this UObject's properties directly, wait until all queued commands are done
	FlushRenderingCommands();
}

void USubUVAnimation::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	SubImages_Horizontal = FMath::Max(SubImages_Horizontal, 1);
	SubImages_Vertical = FMath::Max(SubImages_Vertical, 1);

	TComponentReregisterContext<UParticleSystemComponent> ParticleReregisterContext;
	BeginReleaseResource(BoundingGeometryBuffer);
	// Wait until unregister commands are processed on the RT
	FlushRenderingCommands();

	CacheDerivedData();

	// The SRV is only needed for platforms that can render particles with instancing
	BeginInitResource(BoundingGeometryBuffer);
}

#endif // WITH_EDITOR


void USubUVAnimation::BeginDestroy()
{
	Super::BeginDestroy();

	if (BoundingGeometryBuffer)
	{
		BeginReleaseResource(BoundingGeometryBuffer);
		ReleaseFence.BeginFence();
	}
}

bool USubUVAnimation::IsReadyForFinishDestroy()
{
	bool bReady = Super::IsReadyForFinishDestroy();

	return bReady && ReleaseFence.IsFenceComplete();
}

void USubUVAnimation::FinishDestroy()
{
	delete BoundingGeometryBuffer;

	Super::FinishDestroy();
}


struct FLine2D
{
	FVector2D Position;
	FVector2D Direction;
};

inline float CrossProduct2D(FVector2D Direction0, FVector2D Direction1)
{
	// Also the parallelogram area
	return Direction0.X * Direction1.Y - Direction0.Y * Direction1.X;
}

bool ComputePointIntersectionBetweenLines2D(const FLine2D& Line0, const FLine2D& Line1, FVector2D& OutIntersectPoint)
{
	float d = CrossProduct2D(Line0.Direction, Line1.Direction);

	// Parallel case
	if (FMath::Abs(d) < UE_SMALL_NUMBER)
	{
		return false;
	}

	float t = CrossProduct2D(Line1.Direction, Line0.Position - Line1.Position) / d;

	// Intersects in the wrong direction
	if (t < 0.5f) 
	{
		return false;
	}

	OutIntersectPoint = Line0.Position + t * Line0.Direction;
	return true;
}

bool IsValidUV(FVector2D InUV)
{
	return InUV.X >= 0.0f && InUV.X <= 1.0f
		&& InUV.Y >= 0.0f && InUV.Y <= 1.0f;
}

int32 GetRandomLineIndex(int32 StartIndex, int32 NumLines, FRandomStream& RandomStream)
{
	return FMath::Min(FMath::TruncToInt(RandomStream.GetFraction() * (NumLines - StartIndex)) + StartIndex, NumLines - 1);
}

void FindOptimalPolygon(int32 TargetVertexCount, const TArray<int32>& ConvexHullIndices, const TArray<FVector2D>& PotentialHullVertices, TArray<FVector2D>& OutBoundingVertices)
{
	const int32 VertexCount = FMath::Min(TargetVertexCount, ConvexHullIndices.Num());

	TArray<FLine2D> Lines;
	Lines.Empty(ConvexHullIndices.Num());

	// Precompute the lines of the convex hull
	for (int32 LineIndex = 0; LineIndex < ConvexHullIndices.Num(); LineIndex++)
	{
		FLine2D Line;
		Line.Position = FVector2D(PotentialHullVertices[ConvexHullIndices[LineIndex]]);
		int32 EndVertexIndex = (LineIndex + 1) % ConvexHullIndices.Num();
		Line.Direction = FVector2D(PotentialHullVertices[ConvexHullIndices[EndVertexIndex]]) - Line.Position;
		Lines.Add(Line);
	}

	OutBoundingVertices.Empty(VertexCount);
	OutBoundingVertices.AddZeroed(VertexCount);

	if (VertexCount == 4)
	{
		float MinArea = FLT_MAX;

		// Brute force search through all combinations of convex hull edges for the polygon with the smallest area
		for (int32 x = 0; x < Lines.Num(); x++)
		{
			for (int32 y = x + 1; y < Lines.Num(); y++)
			{
				FVector2D V0;
				// Compute the position where the two edges meet, 
				// but only continue if they do meet (required for valid polygon to be formed,
				// and if the intersection is inside the UV space (required to avoid mapping parts of the texture outside the current subimage)
				if (ComputePointIntersectionBetweenLines2D(Lines[x], Lines[y], V0) && IsValidUV(V0))
				{
					for (int32 z = y + 1; z < Lines.Num(); z++)
					{
						FVector2D V1;
						if (ComputePointIntersectionBetweenLines2D(Lines[y], Lines[z], V1) && IsValidUV(V1))
						{
							for (int32 w = z + 1; w < Lines.Num(); w++)
							{
								FVector2D V2;
								if (ComputePointIntersectionBetweenLines2D(Lines[z], Lines[w], V2) && IsValidUV(V2))
								{
									FVector2D V3;
									if (ComputePointIntersectionBetweenLines2D(Lines[w], Lines[x], V3) && IsValidUV(V3))
									{
										FVector2D U0 = V1 - V0;
										FVector2D U1 = V2 - V0;
										FVector2D U2 = V3 - V0;

										const float Area = 
											(U0.Y * U1.X - U0.X * U1.Y) +
											(U1.Y * U2.X - U1.X * U2.Y);

										if (Area < MinArea)
										{
											MinArea = Area;
											OutBoundingVertices[0] = V0;
											OutBoundingVertices[1] = V1;
											OutBoundingVertices[2] = V2;
											OutBoundingVertices[3] = V3;
										}
									}
								}
							}
						}
					}
				}
			}
		}

		if (MinArea == FLT_MAX)
		{
			OutBoundingVertices.Reset();
		}
	}
	else if (VertexCount == 8)
	{
		float MinArea = FLT_MAX;

		const int32 MaxCombinationsForFullSearch = 100000;

		FRandomStream RandomStream(12345);

		// Search a random subset of the possibility space to guarantee reasonable execution time
		for (int32 SampleIndex = 0; SampleIndex < MaxCombinationsForFullSearch; SampleIndex++)
		{
			int32 x = FMath::TruncToInt(RandomStream.GetFraction() * Lines.Num());
			{
				int32 y = GetRandomLineIndex(x + 1, Lines.Num(), RandomStream);
				{
					FVector2D V0;
					if (ComputePointIntersectionBetweenLines2D(Lines[x], Lines[y], V0) && IsValidUV(V0))
					{
						int32 z = GetRandomLineIndex(y + 1, Lines.Num(), RandomStream);
						{
							FVector2D V1;
							if (ComputePointIntersectionBetweenLines2D(Lines[y], Lines[z], V1) && IsValidUV(V1))
							{
								int32 w = GetRandomLineIndex(z + 1, Lines.Num(), RandomStream);
								{
									FVector2D V2;
									if (ComputePointIntersectionBetweenLines2D(Lines[z], Lines[w], V2) && IsValidUV(V2))
									{
										int32 r = GetRandomLineIndex(w + 1, Lines.Num(), RandomStream);
										{
											FVector2D V3;
											if (ComputePointIntersectionBetweenLines2D(Lines[w], Lines[r], V3) && IsValidUV(V3))
											{
												int32 s = GetRandomLineIndex(r + 1, Lines.Num(), RandomStream);
												{
													FVector2D V4;
													if (ComputePointIntersectionBetweenLines2D(Lines[r], Lines[s], V4) && IsValidUV(V4))
													{
														int32 t = GetRandomLineIndex(s + 1, Lines.Num(), RandomStream);
														{
															FVector2D V5;
															if (ComputePointIntersectionBetweenLines2D(Lines[s], Lines[t], V5) && IsValidUV(V5))
															{
																int32 u = GetRandomLineIndex(t + 1, Lines.Num(), RandomStream);
																{
																	FVector2D V6;
																	if (ComputePointIntersectionBetweenLines2D(Lines[t], Lines[u], V6) && IsValidUV(V6))
																	{
																		FVector2D V7;
																		if (ComputePointIntersectionBetweenLines2D(Lines[u], Lines[x], V7) && IsValidUV(V7))
																		{
																			FVector2D U0 = V1 - V0;
																			FVector2D U1 = V2 - V0;
																			FVector2D U2 = V3 - V0;
																			FVector2D U3 = V4 - V0;
																			FVector2D U4 = V5 - V0;
																			FVector2D U5 = V6 - V0;
																			FVector2D U6 = V7 - V0;

																			float Area = 
																				(U0.Y * U1.X - U0.X * U1.Y) +
																				(U1.Y * U2.X - U1.X * U2.Y) +
																				(U2.Y * U3.X - U2.X * U3.Y) +
																				(U3.Y * U4.X - U3.X * U4.Y) +
																				(U4.Y * U5.X - U4.X * U5.Y) +
																				(U5.Y * U6.X - U5.X * U6.Y);

																			if (Area < MinArea)
																			{
																				MinArea = Area;
																				OutBoundingVertices[0] = V0;
																				OutBoundingVertices[1] = V1;
																				OutBoundingVertices[2] = V2;
																				OutBoundingVertices[3] = V3;
																				OutBoundingVertices[4] = V4;
																				OutBoundingVertices[5] = V5;
																				OutBoundingVertices[6] = V6;
																				OutBoundingVertices[7] = V7;
																			}
																		}
																	}
																}
															}
														}
													}
												}
											}
										}
									}
								}
							}
						}
					}
				}
			}
		}

		if (MinArea == FLT_MAX)
		{
			OutBoundingVertices.Reset();
		}
	}
	else
	{
		for (int32 VertexIndex = 0; VertexIndex < VertexCount; VertexIndex++)
		{
			OutBoundingVertices[VertexIndex] = PotentialHullVertices[ConvexHullIndices[VertexIndex]];
		}
	}
}

FIntPoint GNeighbors[] = 
{
	FIntPoint(-1, -1),
	FIntPoint(0, -1),
	FIntPoint(1, -1),
	FIntPoint(1, 0),
	FIntPoint(1, 1),
	FIntPoint(0, 1),
	FIntPoint(-1, 1),
	FIntPoint(-1, 0)
};


bool IsSupportedFormat(ETextureSourceFormat SrcFormat)
{
	return SrcFormat == TSF_G8 || SrcFormat == TSF_R32F || SrcFormat == TSF_BGRA8 || SrcFormat == TSF_RGBA16 || SrcFormat == TSF_RGBA16F || SrcFormat == TSF_RGBA32F;
}

uint32 GetByteSizePerPixel(ETextureSourceFormat SrcFormat)
{
	switch (SrcFormat)
	{
		case TSF_G8:		return 1;
		case TSF_R32F:		return sizeof(float);
		case TSF_BGRA8:		return 4;
		case TSF_RGBA16:	return 8;
		case TSF_RGBA16F:	return sizeof(FFloat16) * 4;
		case TSF_RGBA32F:	return sizeof(float) * 4;
		default:			return 0;
	}
}

bool ComputeOpacityValue(const uint8* BGRA, int32 x, int32 y, int32 TextureSizeX,  EOpacitySourceMode OpacitySourceMode, ETextureSourceFormat SrcFormat, float AlphaThresholdF, uint8 AlphaThreshold8, uint16 AlphaThreshold16)
{
	const int32 Offset = (y * TextureSizeX + x) * GetByteSizePerPixel(SrcFormat);

	if (SrcFormat == TSF_G8)
	{
		const uint32 Opacity = BGRA[Offset];
		return Opacity > AlphaThreshold8;
	}
	else if (SrcFormat == TSF_R32F)
	{
		const float Opacity = *((const float*)(BGRA + Offset));
		return Opacity > AlphaThresholdF;
	}
	else if (SrcFormat == TSF_BGRA8)
	{
		const uint8* BGRA8 = (const uint8*)(BGRA + Offset);
		uint32 Opacity = 255;
		if (OpacitySourceMode == OSM_Alpha)
		{
			Opacity = *(BGRA8 + 3);
		}
		else if (OpacitySourceMode == OSM_RedChannel)
		{
			Opacity = *(BGRA8 + 2);
		}
		else if (OpacitySourceMode == OSM_GreenChannel)
		{
			Opacity = *(BGRA8 + 1);
		}
		else if (OpacitySourceMode == OSM_BlueChannel)
		{
			Opacity = *(BGRA8 + 0);
		}
		else
		{
			uint32 R = *(BGRA8 + 0);
			uint32 G = *(BGRA8 + 1);
			uint32 B = *(BGRA8 + 2);

			Opacity = ((R + G + B) / 3);
		}
		return Opacity > AlphaThreshold8;
	}
	else if (SrcFormat == TSF_RGBA16)
	{
		const uint16* RGBA16 = (const uint16*)(BGRA + Offset);

		uint32 Opacity = 65535;

		if (OpacitySourceMode == OSM_Alpha)
		{
			Opacity = *(RGBA16 + 3);
		}
		else if (OpacitySourceMode == OSM_RedChannel)
		{
			Opacity = *(RGBA16 + 0);
		}
		else if (OpacitySourceMode == OSM_GreenChannel)
		{
			Opacity = *(RGBA16 + 1);
		}
		else if (OpacitySourceMode == OSM_BlueChannel)
		{
			Opacity = *(RGBA16 + 2);
		}
		else
		{
			uint32 R = *(RGBA16 + 0);
			uint32 G = *(RGBA16 + 1);
			uint32 B = *(RGBA16 + 2);

			Opacity = ((R + G + B) / 3);
		}

		return Opacity > AlphaThreshold16;
	}
	else if (SrcFormat == TSF_RGBA16F)
	{
		const FFloat16* RGBAf = (const FFloat16 *)( BGRA + Offset);

		float Opacity = 1.0f;
		if (OpacitySourceMode == OSM_Alpha)
		{
			Opacity = (*(RGBAf + 3)).GetFloat();
		}
		else if (OpacitySourceMode == OSM_RedChannel)
		{
			Opacity = (*(RGBAf + 0)).GetFloat();
		}
		else if (OpacitySourceMode == OSM_GreenChannel)
		{
			Opacity = (*(RGBAf + 1)).GetFloat();
		}
		else if (OpacitySourceMode == OSM_BlueChannel)
		{
			Opacity = (*(RGBAf + 2)).GetFloat();
		}
		else
		{
			float R = (*RGBAf).GetFloat();
			float G = (*(RGBAf + 1)).GetFloat();
			float B = (*(RGBAf + 2)).GetFloat();

			Opacity = (R + G + B) / 3.0f;
		}

		return Opacity > AlphaThresholdF;
	}
	else if (SrcFormat == TSF_RGBA32F)
	{
		const float* RGBAf = (const float*)( BGRA + Offset);

		float Opacity = 1.0f;
		if (OpacitySourceMode == OSM_Alpha)
		{
			Opacity = RGBAf[3];
		}
		else if (OpacitySourceMode == OSM_RedChannel)
		{
			Opacity = RGBAf[0];
		}
		else if (OpacitySourceMode == OSM_GreenChannel)
		{
			Opacity = RGBAf[1];
		}
		else if (OpacitySourceMode == OSM_BlueChannel)
		{
			Opacity = RGBAf[2];
		}
		else
		{
			Opacity = (RGBAf[0] + RGBAf[1] + RGBAf[2]) / 3.0f;
		}

		return Opacity > AlphaThresholdF;
	}
	return true;
}


void FSubUVDerivedData::GetFeedback(UTexture2D* SubUVTexture, int32 SubImages_Horizontal, int32 SubImages_Vertical, ESubUVBoundingVertexCount BoundingMode, float AlphaThreshold, EOpacitySourceMode OpacitySourceMode,
	TArray<FText>& OutErrors, TArray<FText>& OutWarnings, TArray<FText>& OutInfo)
{

#if WITH_EDITORONLY_DATA
	if (SubUVTexture)
	{
		const ETextureSourceFormat SourceFormat = SubUVTexture->Source.GetFormat();
		if (!IsSupportedFormat(SourceFormat))
		{
			FString SupportedFormatsString;
			UEnum* SourceFormatEnum = StaticEnum<ETextureSourceFormat>();
			for (int32 Format=0; Format < TSF_MAX; ++Format)
			{
				if (IsSupportedFormat(ETextureSourceFormat(Format)))
				{
					if (SupportedFormatsString.IsEmpty() == false)
					{
						SupportedFormatsString.Append(TEXT(", "));
					}
					SupportedFormatsString.Append(SourceFormatEnum->GetNameStringByValue(Format));
				}
			}

			OutErrors.Add(
				FText::Format(LOCTEXT("CutoutNotSupported", "Image source format ({0}) is not in supported format for cutouts ({1}). It will be ignored."),
					FText::FromString(SourceFormatEnum->GetNameStringByValue(SourceFormat)),
					FText::FromString(SupportedFormatsString)
				)
			);
		} 
		if (SubUVTexture->Source.GetNumMips() == 0)
		{
			OutErrors.Add(LOCTEXT("CutoutNoMipNotSupported", "Image does not have a 0th Mip Level."));
		}
	}
#endif
}

/** Counts how many neighbors have non-zero alpha. */
int32 ComputeNeighborCount(int32 X, int32 Y, int32 GlobalX, int32 GlobalY, int32 SubImageSizeX, int32 SubImageSizeY, int32 TextureSizeX, const TArray64<uint8>& MipData,  EOpacitySourceMode OpacitySourceMode, ETextureSourceFormat SourceFormat, float AlphaThresholdF, uint8 AlphaThreshold8, uint16 AlphaThreshold16)
{
	int32 NeighborCount = 0;

	for (int32 NeighborIndex = 0; NeighborIndex < UE_ARRAY_COUNT(GNeighbors); NeighborIndex++)
	{
		int32 NeighborX = X + GNeighbors[NeighborIndex].X;
		int32 NeighborY = Y + GNeighbors[NeighborIndex].Y;

		if (NeighborX >= 0 && NeighborX < SubImageSizeX 
			&& NeighborY >= 0 && NeighborY < SubImageSizeY)
		{
			bool bNeighborPasses = ComputeOpacityValue(MipData.GetData(), GlobalX + GNeighbors[NeighborIndex].X, GlobalY + GNeighbors[NeighborIndex].Y, TextureSizeX, OpacitySourceMode, SourceFormat, AlphaThresholdF, AlphaThreshold8, AlphaThreshold16);

			if (bNeighborPasses)
			{
				NeighborCount++;
			}
		}
	}

	return NeighborCount;
}

struct FSubUVFrameData
{
	TArray<FVector2D> BoundingVertices;
};

void FSubUVDerivedData::Build(UTexture2D* SubUVTexture, int32 SubImages_Horizontal, int32 SubImages_Vertical, ESubUVBoundingVertexCount BoundingMode, float AlphaThreshold, EOpacitySourceMode OpacitySourceMode)
{
#if WITH_EDITORONLY_DATA
	FSubUVFrameData DefaultFrame;
	if (BoundingMode == BVC_FourVertices)
	{
		DefaultFrame.BoundingVertices.Empty(4);
		DefaultFrame.BoundingVertices.Add(FVector2D(0, 0));
		DefaultFrame.BoundingVertices.Add(FVector2D(0, 1));
		DefaultFrame.BoundingVertices.Add(FVector2D(1, 1));
		DefaultFrame.BoundingVertices.Add(FVector2D(1, 0));
	}
	else
	{
		DefaultFrame.BoundingVertices.Empty(8);
		DefaultFrame.BoundingVertices.Add(FVector2D(0, 0));
		DefaultFrame.BoundingVertices.Add(FVector2D(0, 1));
		DefaultFrame.BoundingVertices.Add(FVector2D(1, 1));
		DefaultFrame.BoundingVertices.Add(FVector2D(1, 0));
		DefaultFrame.BoundingVertices.Add(FVector2D(1, 0));
		DefaultFrame.BoundingVertices.Add(FVector2D(1, 0));
		DefaultFrame.BoundingVertices.Add(FVector2D(1, 0));
		DefaultFrame.BoundingVertices.Add(FVector2D(1, 0));
	}

	if (SubUVTexture && SubUVTexture->Source.IsValid())
	{
		TArray64<uint8> MipData;
		ETextureSourceFormat SourceFormat = SubUVTexture->Source.GetFormat();
		bool bSuccess = IsSupportedFormat(SourceFormat) && SubUVTexture->Source.GetMipData(MipData, 0);

		const int32 TextureSizeX = SubUVTexture->Source.GetSizeX();
		const int32 TextureSizeY = SubUVTexture->Source.GetSizeY();
		const int32 SubImageSizeX = TextureSizeX / SubImages_Horizontal;
		const int32 SubImageSizeY = TextureSizeY / SubImages_Vertical;
		const float SubImageSizeXFloat = TextureSizeX / (float)SubImages_Horizontal;
		const int32 SubImageSizeYFloat = TextureSizeY / (float)SubImages_Vertical;

		const int32 NumSubImages = SubImages_Horizontal * SubImages_Vertical;
		const uint8 AlphaThreshold8 = FMath::Clamp(FMath::TruncToInt(AlphaThreshold * 255.0f), 0, 255);
		const uint16 AlphaThreshold16 = FMath::Clamp(FMath::TruncToInt(AlphaThreshold * 65535.0f), 0, 65535.0f);

		check(!bSuccess || MipData.Num() == TextureSizeX * TextureSizeY * GetByteSizePerPixel(SourceFormat));

		const int32 TargetNumBoundingVertices = BoundingMode == BVC_FourVertices ? 4 : 8;

		BoundingGeometry.Empty(NumSubImages * TargetNumBoundingVertices);

		for (int32 SubImageY = 0; SubImageY < SubImages_Vertical; SubImageY++)
		{
			for (int32 SubImageX = 0; SubImageX < SubImages_Horizontal; SubImageX++)
			{
				int32 ImageIndex = SubImageY * SubImages_Horizontal + SubImageX;
				int32 NextImageIndex = (ImageIndex + 1) % NumSubImages;
				int32 NextSubImageX = NextImageIndex % NumSubImages;
				int32 NextSubImageY = NextImageIndex / NumSubImages;

				bool bSubImageSuccess = bSuccess;

				if (bSubImageSuccess)
				{
					TArray<FVector2D> PotentialHullVertices;
					PotentialHullVertices.Empty(NumSubImages);

					for (int32 Y = 0; Y < SubImageSizeY; Y++)
					{
						int32 GlobalY = FMath::RoundToInt((float)SubImageY * SubImageSizeYFloat) + Y;
						int32 NextGlobalY = FMath::RoundToInt((float)NextSubImageY * SubImageSizeYFloat) + Y;

						for (int32 X = 0; X < SubImageSizeX; X++)
						{
							int32 GlobalX = FMath::RoundToInt(SubImageX * SubImageSizeXFloat) + X;
							int32 NextGlobalX = FMath::RoundToInt(NextSubImageX * SubImageSizeXFloat) + X;
							bool AlphaValuePasses = ComputeOpacityValue(MipData.GetData(), GlobalX, GlobalY, TextureSizeX, OpacitySourceMode, SourceFormat, AlphaThreshold, AlphaThreshold8, AlphaThreshold16);
							bool NextAlphaValuePasses = ComputeOpacityValue(MipData.GetData(), NextGlobalX, NextGlobalY, TextureSizeX, OpacitySourceMode, SourceFormat, AlphaThreshold, AlphaThreshold8, AlphaThreshold16);

							if (AlphaValuePasses || NextAlphaValuePasses)
							{
								int32 NeighborCount = AlphaValuePasses ? ComputeNeighborCount(X, Y, GlobalX, GlobalY, SubImageSizeX, SubImageSizeY, TextureSizeX, MipData, OpacitySourceMode, SourceFormat, AlphaThreshold, AlphaThreshold8, AlphaThreshold16) : 8;
								int32 NextNeighborCount = NextAlphaValuePasses ? ComputeNeighborCount(X, Y, NextGlobalX, NextGlobalY, SubImageSizeX, SubImageSizeY, TextureSizeX, MipData, OpacitySourceMode, SourceFormat, AlphaThreshold, AlphaThreshold8, AlphaThreshold16) : 8;

								// Points with non-zero alpha that have 5 or more filled in neighbors must be in the solid interior
								if (NeighborCount < 5 || NextNeighborCount < 5)
								{
									// Add a potential hull vertex for any texel position with non-zero alpha in this frame or the next
									PotentialHullVertices.Add(FVector2D(X / (float)SubImageSizeX, Y / (float)SubImageSizeY));
								}
							}
						}
					}
					
					if (PotentialHullVertices.Num() == 0)
					{
						FSubUVFrameData FrameData;

						// No visible regions of the texture, place all verts at 0
						while (FrameData.BoundingVertices.Num() < TargetNumBoundingVertices)
						{							
							FrameData.BoundingVertices.Add(FVector2D::ZeroVector);
						}

						BoundingGeometry.Append(FrameData.BoundingVertices);
					}
					else
					{
						TArray<int32> ConvexHullIndices;
						// Compute the 2d convex hull of texels with non-zero alpha
						ConvexHull2D::ComputeConvexHullLegacy2(PotentialHullVertices, ConvexHullIndices);

						bSubImageSuccess = ConvexHullIndices.Num() >= 3;

						if (bSubImageSuccess)
						{
							FSubUVFrameData FrameData;
							// Find the minimum area polygon with the specified number of vertices from the convex hull's edges
							FindOptimalPolygon(TargetNumBoundingVertices, ConvexHullIndices, PotentialHullVertices, FrameData.BoundingVertices);

							if (FrameData.BoundingVertices.Num() > 0)
							{
								// Repeat the last vertex until we have satisfied the specified vertex count
								while (FrameData.BoundingVertices.Num() < TargetNumBoundingVertices)
								{
									FVector2D Last = FrameData.BoundingVertices.Last();
									FrameData.BoundingVertices.Add(Last);
								}

								BoundingGeometry.Append(FrameData.BoundingVertices);
							}
							else
							{
								bSubImageSuccess = false;
							}
						}
					}
				}

				if (!bSubImageSuccess)
				{
					BoundingGeometry.Append(DefaultFrame.BoundingVertices);
				}
			}
		}

		check(BoundingGeometry.Num() == SubImages_Horizontal * SubImages_Vertical * TargetNumBoundingVertices);
	}
	else
	{
		// No texture set, fill with default vertices
		const int32 TargetNumBoundingVertices = BoundingMode == BVC_FourVertices ? 4 : 8;
		const int32 NumSubImages = SubImages_Horizontal * SubImages_Vertical;

		BoundingGeometry.Empty(NumSubImages * TargetNumBoundingVertices);

		for (int32 SubImageY = 0; SubImageY < SubImages_Vertical; SubImageY++)
		{
			for (int32 SubImageX = 0; SubImageX < SubImages_Horizontal; SubImageX++)
			{
				BoundingGeometry.Append(DefaultFrame.BoundingVertices);
			}
		}
	}
#endif
}
#undef LOCTEXT_NAMESPACE
