// Copyright Epic Games, Inc. All Rights Reserved.

#include "Particles/WorldPSCPool.h"
#include "Engine/World.h"
#include "GameFramework/WorldSettings.h"
#include "Particles/Emitter.h"
#include "Particles/ParticleSystemComponent.h"
#include "ParticleHelper.h"
#include "Particles/ParticleSystem.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(WorldPSCPool)

static float GParticleSystemPoolKillUnusedTime = 180.0f;
static FAutoConsoleVariableRef ParticleSystemPoolKillUnusedTime(
	TEXT("FX.ParticleSystemPool.KillUnusedTime"),
	GParticleSystemPoolKillUnusedTime,
	TEXT("How long a pooled particle component needs to be unused for before it is destroyed.")
);

static int32 GbEnableParticleSystemPooling = 1;
static FAutoConsoleVariableRef bEnableParticleSystemPooling(
	TEXT("FX.ParticleSystemPool.Enable"),
	GbEnableParticleSystemPooling,
	TEXT("How many Particle System Components to preallocate when creating new ones for the pool.")
);

static float GParticleSystemPoolingCleanTime = 30.0f;
static FAutoConsoleVariableRef ParticleSystemPoolingCleanTime(
	TEXT("FX.ParticleSystemPool.CleanTime"),
	GParticleSystemPoolingCleanTime,
	TEXT("How often should the pool be cleaned (in seconds).")
);

void FPSCPool::Cleanup()
{
	for (FPSCPoolElem& Elem : FreeElements)
	{
		if (Elem.PSC)
		{
			Elem.PSC->PoolingMethod = EPSCPoolMethod::None;//Reset so we don't trigger warnings about destroying pooled PSCs.
			Elem.PSC->DestroyComponent();
		}
		else
		{
			UE_LOGF(LogParticles, Error, "Free element in the WorldPSCPool was null. Someone must be keeping a reference to a PSC that has been freed to the pool and then are manually destroying it.");
		}
	}

	FreeElements.Empty();

#if ENABLE_PSC_POOL_DEBUGGING
	InUseComponents_Auto.Empty();
	InUseComponents_Manual.Empty();
#endif
}

UParticleSystemComponent* FPSCPool::Acquire(UWorld* World, UParticleSystem* Template, EPSCPoolMethod PoolingMethod)
{
	check(GbEnableParticleSystemPooling);
	check(PoolingMethod != EPSCPoolMethod::None);

	FPSCPoolElem RetElem;
	if (FreeElements.Num())
	{
		RetElem = FreeElements.Pop(EAllowShrinking::No);
		check(RetElem.PSC->Template == Template);
		check(IsValid(RetElem.PSC));

		//Reset visibility in case the component was reclaimed by the pool while invisible.
		RetElem.PSC->SetVisibility(true);
		RetElem.PSC->SetHiddenInGame(false);

		if (RetElem.PSC->GetWorld() != World)
		{
			// Rename the PSC to move it into the current PersistentLevel - it may have been spawned in one
			// level but is now needed in another level.
			RetElem.PSC->Rename(nullptr, World);
		}
	}
	else
	{
		//None in the pool so create a new one.
		AActor* OuterActor = World->GetWorldSettings();
		UObject* OuterObject = OuterActor ? static_cast<UObject*>(OuterActor) : static_cast<UObject*>(World);

		RetElem.PSC = NewObject<UParticleSystemComponent>(OuterObject);
		RetElem.PSC->bAutoDestroy = false;//<<< We don't auto destroy. We'll just periodically clear up the pool.
		RetElem.PSC->SecondsBeforeInactive = 0.0f;
		RetElem.PSC->bAutoActivate = false;
		RetElem.PSC->SetTemplate(Template);
		RetElem.PSC->bOverrideLODMethod = false;
		RetElem.PSC->bAllowRecycling = true;
	}

	RetElem.PSC->PoolingMethod = PoolingMethod;

#if ENABLE_PSC_POOL_DEBUGGING
	if (PoolingMethod == EPSCPoolMethod::AutoRelease)
	{
		InUseComponents_Auto.Emplace(RetElem.PSC);
	}
	else if (PoolingMethod == EPSCPoolMethod::ManualRelease)
	{
		InUseComponents_Manual.Emplace(RetElem.PSC);
	}

	MaxUsed = FMath::Max(MaxUsed, InUseComponents_Manual.Num() + InUseComponents_Auto.Num());
#endif 

	//UE_LOGF(LogParticles, Log, "FPSCPool::Acquire() - World: %p - PSC: %p - Sys: %ls", World, RetElem.PSC, *Template->GetFullName());

	return RetElem.PSC;
}

void FPSCPool::Reclaim(UParticleSystemComponent* PSC, const float CurrentTimeSeconds)
{
	check(PSC);

	//UE_LOGF(LogParticles, Log, "FPSCPool::Reclaim() - World: %p - PSC: %p - Sys: %ls", PSC->GetWorld(), PSC, *PSC->Template->GetFullName());

#if ENABLE_PSC_POOL_DEBUGGING
	bool bWasInList = false;
	if (PSC->PoolingMethod == EPSCPoolMethod::AutoRelease)
	{
		bWasInList = InUseComponents_Auto.RemoveSingleSwap(PSC) > 0;
	}
	else if (PSC->PoolingMethod == EPSCPoolMethod::ManualRelease)
	{
		bWasInList = InUseComponents_Manual.RemoveSingleSwap(PSC) > 0;
	}

	if (!bWasInList)
	{
		UE_LOGF(LogParticles, Error, "World Particle System Pool is reclaiming a component that is not in it's InUse list!");
	}
#endif

	//Don't add back to the pool if we're no longer pooling or we've hit our max resident pool size.
	if (GbEnableParticleSystemPooling != 0 && FreeElements.Num() < (int32)PSC->Template->MaxPoolSize)
	{
		PSC->bHasBeenActivated = false;//Clear this flag so we re register with the significance manager on next activation.
		PSC->DetachFromComponent(FDetachmentTransformRules::KeepWorldTransform); // When detaching, maintain world position for optimization purposes
		PSC->SetRelativeScale3D(FVector(1.f)); // Reset scale to avoid future uses of this PSC having incorrect scale
		PSC->SetAbsolute(); // Clear out Absolute settings to defaults
		PSC->UnregisterComponent();
		PSC->SetCastShadow(false);

		PSC->OnParticleSpawn.Clear();
		PSC->OnParticleBurst.Clear();
		PSC->OnParticleDeath.Clear();
		PSC->OnParticleCollide.Clear();

		//Clear some things so that this PSC is re-used as though it were brand new.
		PSC->bWasActive = false;

		//Clear out instance parameters.
		PSC->InstanceParameters.Reset();
		
		//Ensure a small cull distance doesn't linger to future users.
		PSC->SetCullDistance(FLT_MAX);

		PSC->PoolingMethod = EPSCPoolMethod::FreeInPool;
		FreeElements.Push(FPSCPoolElem(PSC, CurrentTimeSeconds));
	}
	else
	{
		//We've stopped pooling while some effects were in flight so ensure they're destroyed now.
		PSC->PoolingMethod = EPSCPoolMethod::None;//Reset so we don't trigger warnings about destroying pooled PSCs.
		PSC->DestroyComponent();
	}
}

void FPSCPool::KillUnusedComponents(float KillTime, UParticleSystem* Template)
{
	int32 i = 0;
	while (i < FreeElements.Num())
	{
		if (FreeElements[i].LastUsedTime < KillTime)
		{
			UParticleSystemComponent* PSC = FreeElements[i].PSC;
			if (PSC)
			{
				PSC->PoolingMethod = EPSCPoolMethod::None;//Reset so we don't trigger warnings about destroying pooled PSCs.
				PSC->DestroyComponent();
			}

			FreeElements.RemoveAtSwap(i, EAllowShrinking::No);
		}
		else
		{
			++i;
		}
	}
	FreeElements.Shrink();

#if ENABLE_PSC_POOL_DEBUGGING
	// Clean up any in use components that have been cleared out from under the pool. This could happen in someone manually destroys a component for example.
	if (InUseComponents_Manual.RemoveAllSwap([](const TWeakObjectPtr<UParticleSystemComponent>& WeakPtr) { return !WeakPtr.IsValid(); }) > 0)
	{
		UE_LOGF(LogParticles, Log, "Manual Pooled PSC has been destroyed! Possibly via a DestroyComponent() call. You should not destroy these but rather call ReleaseToPool on the component so it can be re-used. |\t System: %ls", *Template->GetFullName());
	}

	if (InUseComponents_Auto.RemoveAllSwap([](const TWeakObjectPtr<UParticleSystemComponent>& WeakPtr) { return !WeakPtr.IsValid(); }) > 0)
	{
		UE_LOGF(LogParticles, Log, "Auto Pooled PSC has been destroyed! Possibly via a DestroyComponent() call. You should not destroy these manually. Just deactivate them and allow then to be reclaimed by the pool automatically. |\t System: %ls", *Template->GetFullName());
	}
#endif
}

//////////////////////////////////////////////////////////////////////////

FWorldPSCPool::FWorldPSCPool()
	: LastParticleSytemPoolCleanTime(0.0f)
	, CachedWorldTime(0.0f)
{

}

FWorldPSCPool::~FWorldPSCPool()
{
	Cleanup(nullptr);
}

void FWorldPSCPool::Cleanup(UWorld* World)
{
	for (auto& Pool : WorldParticleSystemPools)
	{
		Pool.Value.Cleanup();
	}

	WorldParticleSystemPools.Empty();

	// If we passed in a world make sure we cleanup any pooled components for that world
	// This is generally used on world cleanup to ensure all pooled components are cleaned up as they can not be returned back to the component pool.
	if (World != nullptr)
	{
		if (AWorldSettings* WorldSettings = World->GetWorldSettings(false, false))
		{
			TArray<UParticleSystemComponent*> ActiveComponents;
			WorldSettings->GetComponents(ActiveComponents);
			for (UParticleSystemComponent* Comp : ActiveComponents)
			{
				if ((Comp->PoolingMethod == EPSCPoolMethod::AutoRelease) || (Comp->PoolingMethod == EPSCPoolMethod::ManualRelease))
				{
					Comp->PoolingMethod = EPSCPoolMethod::None;
					Comp->DestroyComponent();
				}
			}
		}
	}
}

UParticleSystemComponent* FWorldPSCPool::CreateWorldParticleSystem(UParticleSystem* Template, UWorld* World, EPSCPoolMethod PoolingMethod)
{
	check(IsInGameThread());
	check(World);
	if (!Template)
	{
		UE_LOGF(LogParticles, Warning, "Attempted CreateWorldParticleSystem() with a NULL Template!");
		return nullptr;
	}

	if (World->bIsTearingDown)
	{
		UE_LOGF(LogParticles, Warning, "Failed to create pooled particle system as we are tearing the world down.");
		return nullptr;
	}

	UParticleSystemComponent* PSC = nullptr;
	if (GbEnableParticleSystemPooling != 0)
	{
		if (Template->CanBePooled())
		{
			FPSCPool& PSCPool = WorldParticleSystemPools.FindOrAdd(Template);
			PSC = PSCPool.Acquire(World, Template, PoolingMethod);
		}
	}
	else
	{
		WorldParticleSystemPools.Empty();//Ensure the pools are cleared out if we've just switched to not pooling.
	}

	if(PSC == nullptr)
	{
		//Create a new auto destroy system if we're not pooling.
		PSC = NewObject<UParticleSystemComponent>(World);
		PSC->bAutoDestroy = true;
		PSC->SecondsBeforeInactive = 0.0f;
		PSC->bAutoActivate = false;
		PSC->SetTemplate(Template);
		PSC->bOverrideLODMethod = false;
	}

	check(PSC);
	return PSC;
}

/** Called when an in-use particle component is finished and wishes to be returned to the pool. */
void FWorldPSCPool::ReclaimWorldParticleSystem(UParticleSystemComponent* PSC)
{
	check(IsInGameThread());
	
	//If this component has been already destroyed we don't add it back to the pool. Just warn so users can fixup.
	if (!IsValid(PSC))
	{
		UE_LOGF(LogParticles, Log, "Pooled PSC has been destroyed! Possibly via a DestroyComponent() call. You should not destroy components set to auto destroy manually. \nJust deactivate them and allow them to destroy themselves or be reclaimed by the pool if pooling is enabled. | PSC: %p |\t System: %ls", PSC, *PSC->Template->GetFullName());
		return;
	}

	if (GbEnableParticleSystemPooling)
	{
		float CurrentTime = PSC->GetWorld()->GetTimeSeconds();

		//Periodically clear up the pools.
		if (CurrentTime - LastParticleSytemPoolCleanTime > GParticleSystemPoolingCleanTime)
		{
			LastParticleSytemPoolCleanTime = CurrentTime;
			for (auto& Pair : WorldParticleSystemPools)
			{
				Pair.Value.KillUnusedComponents(CurrentTime - GParticleSystemPoolKillUnusedTime, PSC->Template);
			}
		}
		
		FPSCPool* PSCPool = WorldParticleSystemPools.Find(PSC->Template);
		if (!PSCPool)
		{
			UE_LOGF(LogParticles, Warning, "WorldPSC Pool trying to reclaim a system for which it doesn't have a pool! Likely because SetTemplate() has been called on this PSC. | World: %p | PSC: %p | Sys: %ls", PSC->GetWorld(), PSC, *PSC->Template->GetFullName());
			//Just add the new pool and reclaim to that one.
			PSCPool = &WorldParticleSystemPools.Add(PSC->Template);
		}

		PSCPool->Reclaim(PSC, CurrentTime);
	}
	else
	{
		PSC->DestroyComponent();
	}
}

void FWorldPSCPool::Dump()
{
#if ENABLE_PSC_POOL_DEBUGGING
	FString DumpStr;

	uint32 TotalMemUsage = 0;
	for (auto& Pair : WorldParticleSystemPools)
	{
		UParticleSystem* System = Pair.Key;
		FPSCPool& Pool = Pair.Value;		
		uint32 FreeMemUsage = 0;
		for (FPSCPoolElem& Elem : Pool.FreeElements)
		{
			if (ensureAlways(Elem.PSC))
			{
				FreeMemUsage += Elem.PSC->GetApproxMemoryUsage();
			}
		}
		uint32 InUseMemUsage = 0;
		for (auto WeakPSC : Pool.InUseComponents_Auto)
		{
			UParticleSystemComponent* PSC = WeakPSC.Get();
			if (ensureAlways(PSC))
			{
				InUseMemUsage += PSC->GetApproxMemoryUsage();				
			}
		}
		for (auto WeakPSC : Pool.InUseComponents_Manual)
		{
			UParticleSystemComponent* PSC = WeakPSC.Get();
			if (ensureAlways(PSC))
			{
				InUseMemUsage += PSC->GetApproxMemoryUsage();
			}
		}

		TotalMemUsage += FreeMemUsage;
		TotalMemUsage += InUseMemUsage;

		DumpStr += FString::Printf(TEXT("Free: %d (%uB) \t|\t Used(Auto - Manual): %d - %d (%uB) \t|\t MaxUsed: %d \t|\t System: %s\n"), Pool.FreeElements.Num(), FreeMemUsage, Pool.InUseComponents_Auto.Num(), Pool.InUseComponents_Manual.Num(), InUseMemUsage, Pool.MaxUsed, *System->GetFullName());
	}

	UE_LOGF(LogParticles, Log, "***************************************");
	UE_LOGF(LogParticles, Log, "*Particle System Pool Info - Total Mem = %.2fMB*", TotalMemUsage / 1024.0f / 1024.0f);
	UE_LOGF(LogParticles, Log, "***************************************");
	UE_LOGF(LogParticles, Log, "%ls", *DumpStr);
	UE_LOGF(LogParticles, Log, "***************************************");

#endif
}

void DumpPooledWorldParticleSystemInfo(UWorld* World)
{
	check(World);
	World->GetPSCPool().Dump();
}

FAutoConsoleCommandWithWorld DumpPSCPoolInfoCommand(
	TEXT("fx.DumpPSCPoolInfo"),
	TEXT("Dump Particle System Pooling Info"),
	FConsoleCommandWithWorldDelegate::CreateStatic(&DumpPooledWorldParticleSystemInfo)
);

