// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	Pawn.cpp: APawn AI implementation
	This contains both C++ methods (movement and reachability), as well as 
	some AI related natives
=============================================================================*/

#include "GameFramework/Pawn.h"
#include "Engine/Level.h"
#include "Engine/Player.h"
#include "GameFramework/DamageType.h"
#include "Engine/GameInstance.h"
#include "AI/NavigationSystemBase.h"
#include "Engine/Engine.h"
#include "Engine/Canvas.h"
#include "Components/SkeletalMeshComponent.h"
#include "Kismet/GameplayStatics.h"
#include "Misc/PackageName.h"
#include "GameFramework/PawnMovementComponent.h"
#include "Net/UnrealNetwork.h"
#include "DisplayDebugHelpers.h"
#include "GameFramework/MovementInterface.h"
#include "Engine/InputDelegateBinding.h"
#include "Interfaces/NetworkPredictionInterface.h"
#include "Interfaces/MovementBaseInterface.h"
#include "GameFramework/PlayerState.h"
#include "Components/PawnNoiseEmitterComponent.h"
#include "GameFramework/GameNetworkManager.h"
#include "GameFramework/InputSettings.h"
#include "PhysicsEngine/PhysicsSettings.h"
#include "Engine/DemoNetDriver.h"
#include "Misc/EngineNetworkCustomVersion.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(Pawn)

DEFINE_LOG_CATEGORY(LogDamage);
DEFINE_LOG_CATEGORY_STATIC(LogPawn, Warning, All);

FOnPawnBeginPlay APawn::OnPawnBeginPlay;

namespace UE::Gameplay::CVars
{
	extern bool bAlwaysNotifyClientOnControllerChange;
}

APawn::APawn(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	PrimaryActorTick.bCanEverTick = true;
	PrimaryActorTick.TickGroup = TG_PrePhysics;

	AutoPossessAI = EAutoPossessAI::PlacedInWorld;

	if (HasAnyFlags(RF_ClassDefaultObject) && GetClass() == APawn::StaticClass())
	{
		bool bLoadPluginClass = true;

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
		// disabling engine plugins will cause loader warnings, this was added because it added non-valuable warnings for things like basic commandlets
		if (FParse::Param(FCommandLine::Get(), TEXT("NoEnginePlugins")))
		{
			bLoadPluginClass = false;
		}
#endif

		FString AIControllerClassName = GetDefault<UEngine>()->AIControllerClassName.ToString();
		check(FPackageName::IsValidObjectPath(AIControllerClassName));
		// resolve the name to a UClass
		AIControllerClass = FindObject<UClass>(nullptr, *AIControllerClassName);

		// if we failed to resolve, and plugins are expected to be loaded, proceed with loading it
		if (AIControllerClass == nullptr && bLoadPluginClass)
		{
			// WARNING: This line is why the AISupport plugin has to load the AIModule before UObject initialization, otherwise this load fails and CDOs are corrupt in the editor
			AIControllerClass = LoadClass<AController>(nullptr, *AIControllerClassName, nullptr, LOAD_None, nullptr);
		}
	}
	else
	{
		AIControllerClass = ((APawn*)APawn::StaticClass()->GetDefaultObject())->AIControllerClass;
	}
	SetCanBeDamaged(true);
	
	SetRemoteRoleForBackwardsCompat(ROLE_SimulatedProxy);
	bReplicates = true;
	NetPriority = 3.0f;
	SetNetUpdateFrequency(100.f);
	SetReplicatingMovement(true);
	BaseEyeHeight = 64.0f;
	AllowedYawError = 10.99f;
	bCollideWhenPlacing = true;
	SpawnCollisionHandlingMethod = ESpawnActorCollisionHandlingMethod::AdjustIfPossibleButDontSpawnIfColliding;
	bGenerateOverlapEventsDuringLevelStreaming = true;
	bProcessingOutsideWorldBounds = false;
	bIsLocalViewTarget = false;

	bUseControllerRotationPitch = false;
	bUseControllerRotationYaw = false;
	bUseControllerRotationRoll = false;

	bInputEnabled = true;

	GetReplicatedMovement_Mutable().LocationQuantizationLevel = EVectorQuantization::RoundTwoDecimals;
}

void APawn::PreInitializeComponents()
{
	Super::PreInitializeComponents();

	if (GetInstigator() == nullptr)
	{
		SetInstigator(this);
	}

	if (AutoPossessPlayer != EAutoReceiveInput::Disabled && GetNetMode() != NM_Client )
	{
		const int32 PlayerIndex = int32(AutoPossessPlayer.GetValue()) - 1;

		APlayerController* PC = UGameplayStatics::GetPlayerController(this, PlayerIndex);
		if (PC)
		{
			PC->Possess(this);
		}
		else
		{
			GetWorld()->PersistentLevel->RegisterActorForAutoReceiveInput(this, PlayerIndex);
		}
	}

	UpdateNavigationRelevance();
}

void APawn::PostInitializeComponents()
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_Pawn_PostInitComponents);

	Super::PostInitializeComponents();
	
	if (IsValidChecked(this))
	{
		UWorld* World = GetWorld();

		// Automatically add Controller to AI Pawns if we are allowed to.
		if (AutoPossessPlayer == EAutoReceiveInput::Disabled
			&& AutoPossessAI != EAutoPossessAI::Disabled && GetController() == nullptr && GetNetMode() != NM_Client
#if WITH_EDITOR
			&& (GIsEditor == false || World->IsGameWorld())
#endif // WITH_EDITOR
			)
		{
			const bool bPlacedInWorld = (World->bStartup);
			if ((AutoPossessAI == EAutoPossessAI::PlacedInWorldOrSpawned) ||
				(AutoPossessAI == EAutoPossessAI::PlacedInWorld && bPlacedInWorld) ||
				(AutoPossessAI == EAutoPossessAI::Spawned && !bPlacedInWorld))
			{
				SpawnDefaultController();
			}
		}

		// update movement component's nav agent values
		UpdateNavAgent();
	}
}

void APawn::PostLoad()
{
	Super::PostLoad();

	// A pawn should never have this enabled, so we'll aggressively disable it if it did occur.
	AutoReceiveInput = EAutoReceiveInput::Disabled;
}

void APawn::PostRegisterAllComponents()
{
	Super::PostRegisterAllComponents();

	UpdateNavAgent();
}

void APawn::BeginPlay()
{
	Super::BeginPlay();

	OnPawnBeginPlay.Broadcast(this);
}

UPawnMovementComponent* APawn::GetMovementComponent() const
{
	return FindComponentByClass<UPawnMovementComponent>();
}

void APawn::UpdateNavAgent()
{
	INavMovementInterface* NavMovementInterface = FindComponentByInterface<INavMovementInterface>();
	
	//// Update Nav Agent props with collision component's setup if it's not set yet
	if (RootComponent != nullptr && NavMovementInterface != nullptr)
	{
		const FNavMovementProperties* NavMovementProperties = NavMovementInterface->GetNavMovementProperties();
		if (NavMovementProperties && NavMovementProperties->bUpdateNavAgentWithOwnersCollision != 0)
		{
			RootComponent->UpdateBounds();
			NavMovementInterface->UpdateNavAgent(*this);
		}
	}
}

void APawn::SetCanAffectNavigationGeneration(bool bNewValue, bool bForceUpdate)
{
	if (bCanAffectNavigationGeneration != bNewValue || bForceUpdate)
	{
		bCanAffectNavigationGeneration = bNewValue;

		// update components 
		UpdateNavigationRelevance();

		// update entries in navigation system
		FNavigationSystem::UpdateActorAndComponentData(*this);
	}
}

void APawn::PawnStartFire(uint8 FireModeNum) {}

AActor* APawn::GetMovementBaseActor(const APawn* Pawn)
{
	return Cast<AActor>(GetMovementBaseOwner(Pawn));
}

UObject* APawn::GetMovementBaseOwner(const APawn* Pawn)
{
	const FMovementBaseInterfaceData* MovementBaseInterfaceData = (Pawn ? Pawn->GetMovementBaseInterfaceData() : nullptr);
	if (MovementBaseInterfaceData && MovementBaseInterfaceData->IsValid())
	{
		return MovementBaseInterfaceData->GetMovementBaseObjectOwner();
	}
	
	return nullptr;
}

bool APawn::CanBeBaseForCharacter(class APawn* APawn) const
{
	UPrimitiveComponent* RootPrimitive = Cast<UPrimitiveComponent>(GetRootComponent());
	if (RootPrimitive && RootPrimitive->CanCharacterStepUpOn != ECB_Owner)
	{
		return RootPrimitive->CanCharacterStepUpOn == ECB_Yes;
	}

	return Super::CanBeBaseForCharacter(APawn);
}

FVector APawn::GetVelocity() const
{
	if(GetRootComponent() && GetRootComponent()->IsSimulatingPhysics())
	{
		return GetRootComponent()->GetComponentVelocity();
	}

	if (const UPawnMovementComponent* MovementComponent = GetMovementComponent())
	{
		return MovementComponent->Velocity;
	}
	
	if (const IMovementInterface* MovementInterface = FindComponentByInterface<IMovementInterface>())
	{
		return MovementInterface->GetVelocity();
	}
	
	return FVector::ZeroVector;
}

bool APawn::IsLocallyControlled() const
{
	return ( GetController() && GetController()->IsLocalController() );
}

FPlatformUserId APawn::GetPlatformUserId() const
{
	if (const APlayerController* PC = Cast<APlayerController>(GetController()))
	{
		return PC->GetPlatformUserId();
	}
	return PLATFORMUSERID_NONE;
}

bool APawn::IsPlayerControlled() const
{
	return GetPlayerState() && !GetPlayerState()->IsABot();
}

bool APawn::IsBotControlled() const
{
	return GetPlayerState() && GetPlayerState()->IsABot();
}

bool APawn::ReachedDesiredRotation()
{
	// Only base success on Yaw 
	FRotator DesiredRotation = GetController() ? GetController()->GetDesiredRotation() : GetActorRotation();
	float YawDiff = FMath::Abs(FRotator::ClampAxis(DesiredRotation.Yaw) - FRotator::ClampAxis(GetActorRotation().Yaw));
	return ( (YawDiff < AllowedYawError) || (YawDiff > 360.f - AllowedYawError) );
}

float APawn::GetDefaultHalfHeight() const
{
	USceneComponent* DefaultRoot = GetClass()->GetDefaultObject<APawn>()->RootComponent;
	if (DefaultRoot)
	{
		float Radius, HalfHeight;
		DefaultRoot->UpdateBounds(); // Since it's the default object, it wouldn't have been registered to ever do this.
		DefaultRoot->CalcBoundingCylinder(Radius, HalfHeight);
		return HalfHeight;
	}
	else
	{
		// This will probably fail to return anything useful, since default objects won't have registered components,
		// but at least it will spit out a warning if so.
		return GetClass()->GetDefaultObject<APawn>()->GetSimpleCollisionHalfHeight();
	}
}

void APawn::SetRemoteViewPitch(double NewRemoteViewPitch)
{
	// Compress pitch to a 16-bit short
	RemoteViewPitch16 = FRotator::CompressAxisToShort(NewRemoteViewPitch);
}


UPawnNoiseEmitterComponent* APawn::GetPawnNoiseEmitterComponent() const
{
	 UPawnNoiseEmitterComponent* NoiseEmitterComponent = FindComponentByClass<UPawnNoiseEmitterComponent>();
	 if (!NoiseEmitterComponent && GetController())
	 {
		 NoiseEmitterComponent = GetController()->FindComponentByClass<UPawnNoiseEmitterComponent>();
	 }

	 return NoiseEmitterComponent;
}

FVector APawn::GetGravityDirection() const
{
	return FVector(0.f,0.f,-1.f);
}

FQuat APawn::GetGravityTransform() const
{
	return FQuat::Identity;
}

bool APawn::ShouldTickIfViewportsOnly() const 
{ 
	return IsLocallyControlled() && Cast<APlayerController>(GetController()); 
}

FVector APawn::GetPawnViewLocation() const
{
	return GetActorLocation() + FVector(0.f,0.f,BaseEyeHeight);
}

FRotator APawn::GetViewRotation() const
{
	if (GetController() != nullptr)
	{
		return GetController()->GetControlRotation();
	}
	else if (GetLocalRole() < ROLE_Authority)
	{
		// check if being spectated
		for (FConstPlayerControllerIterator Iterator = GetWorld()->GetPlayerControllerIterator(); Iterator; ++Iterator)
		{
			APlayerController* PlayerController = Iterator->Get();
			if (PlayerController &&
				PlayerController->PlayerCameraManager &&
				PlayerController->PlayerCameraManager->GetViewTargetPawn() == this)
			{
				return PlayerController->BlendedTargetViewRotation;
			}
		}
	}

	return GetActorRotation();
}

void APawn::SpawnDefaultController()
{
	if (GetController() != nullptr || GetNetMode() == NM_Client)
	{
		return;
	}

	if (AIControllerClass != nullptr)
	{
		FActorSpawnParameters SpawnInfo;
		SpawnInfo.Instigator = GetInstigator();
		SpawnInfo.SpawnCollisionHandlingOverride = ESpawnActorCollisionHandlingMethod::AlwaysSpawn;
		SpawnInfo.OverrideLevel = GetLevel();
		SpawnInfo.ObjectFlags |= RF_Transient;	// We never want to save AI controllers into a map
		AController* NewController = GetWorld()->SpawnActor<AController>(AIControllerClass, GetActorLocation(), GetActorRotation(), SpawnInfo);
		if (NewController != nullptr)
		{
			// if successful will result in setting this->Controller 
			// as part of possession mechanics
			NewController->Possess(this);
		}
	}
}

void APawn::TurnOff()
{
	if (GetLocalRole() == ROLE_Authority)
	{
		SetReplicates(true);
	}
	
	// do not block anything, just ignore
	SetActorEnableCollision(false);

	UPawnMovementComponent* MovementComponent = GetMovementComponent();
	if (MovementComponent)
	{
		MovementComponent->StopMovementImmediately();
		MovementComponent->SetComponentTickEnabled(false);
	}

	DisableComponentsSimulatePhysics();
}

void APawn::BecomeViewTarget(APlayerController* PC)
{
	Super::BecomeViewTarget(PC);

	if (GetNetMode() != NM_Client)
	{
		ForceNetUpdate();
	}

	if (GetNetMode() != NM_DedicatedServer)
	{
		bIsLocalViewTarget = GetLocalViewingPlayerController() != nullptr;
	}
}

void APawn::EndViewTarget(APlayerController* PC)
{
	Super::EndViewTarget(PC);

	if (GetNetMode() != NM_DedicatedServer)
	{
		// are any other PCs viewing this pawn before we set it to false
		bIsLocalViewTarget = GetLocalViewingPlayerController() != nullptr;
	}
}

bool APawn::IsLocallyViewed() const
{
	return bIsLocalViewTarget;
}

bool APawn::IsLocalPlayerControllerViewingAPawn() const
{
	if (UWorld* World = GetWorld())
	{
		for (FConstPlayerControllerIterator Iterator = World->GetPlayerControllerIterator(); Iterator; ++Iterator)
		{
			APlayerController* PlayerController = Iterator->Get();
			if (PlayerController &&
				PlayerController->IsLocalController() &&
				PlayerController->GetViewTarget() != nullptr &&
				PlayerController->GetViewTarget() != PlayerController)
			{
				return true;
			}
		}
	}

	return false;
}

APlayerController* APawn::GetLocalViewingPlayerController() const
{
	if (UWorld* World = GetWorld())
	{
		for (FConstPlayerControllerIterator Iterator = World->GetPlayerControllerIterator(); Iterator; ++Iterator)
		{
			APlayerController* PlayerController = Iterator->Get();
			if (PlayerController && PlayerController->IsLocalController() && PlayerController->GetViewTarget() == this)
			{
				return PlayerController;
			}
		}
	}

	return nullptr;
}

void APawn::PawnClientRestart()
{
	Restart();

	APlayerController* PC = Cast<APlayerController>(GetController());
	if (PC && PC->IsLocalController())
	{
		// Handle camera possession
		if (PC->bAutoManageActiveCameraTarget)
		{
			PC->AutoManageActiveCameraTarget(this);
		}

		// Set up player input component, if there isn't one already.
		if (InputComponent == nullptr)
		{
			InputComponent = CreatePlayerInputComponent();
			if (InputComponent)
			{
				SetupPlayerInputComponent(InputComponent);
				InputComponent->RegisterComponent();
			PRAGMA_DISABLE_DEPRECATION_WARNINGS
				if (UInputDelegateBinding::SupportsInputDelegate(GetClass()))
			PRAGMA_ENABLE_DEPRECATION_WARNINGS	
				{
					InputComponent->bBlockInput = bBlockInput;
					UInputDelegateBinding::BindInputDelegatesWithSubojects(this, InputComponent);
				}
			}
		}
	}
}

void APawn::NotifyRestarted()
{
	ReceiveRestarted();
	ReceiveRestartedDelegate.Broadcast(this);
}

void APawn::DispatchRestart(bool bCallClientRestart)
{
	if (bCallClientRestart)
	{
		// This calls Restart()
		PawnClientRestart();
	}
	else
	{
		Restart();
	}

	NotifyRestarted();
}

void APawn::Destroyed()
{
	DetachFromControllerPendingDestroy();
	Super::Destroyed();
}

void APawn::EndPlay(const EEndPlayReason::Type EndPlayReason)
{
	// Only do this once, to not be redundant with Destroyed().
	if (EndPlayReason != EEndPlayReason::Destroyed)
	{
		DetachFromControllerPendingDestroy();
	}
	
	Super::EndPlay(EndPlayReason);
}

bool APawn::ShouldTakeDamage(float Damage, FDamageEvent const& DamageEvent, AController* EventInstigator, AActor* DamageCauser) const
{
	if ((GetLocalRole() < ROLE_Authority) || !CanBeDamaged() || !GetWorld()->GetAuthGameMode() || (Damage == 0.f))
	{
		return false;
	}

	return true;
}

float APawn::TakeDamage(float Damage, FDamageEvent const& DamageEvent, AController* EventInstigator, AActor* DamageCauser)
{
	if (!ShouldTakeDamage(Damage, DamageEvent, EventInstigator, DamageCauser))
	{
		return 0.f;
	}

	// do not modify damage parameters after this
	const float ActualDamage = Super::TakeDamage(Damage, DamageEvent, EventInstigator, DamageCauser);

	// respond to the damage
	if (ActualDamage != 0.f)
	{
		if ( EventInstigator && EventInstigator != GetController())
		{
			LastHitBy = EventInstigator;
		}
	}

	return ActualDamage;
}

bool APawn::IsControlled() const
{
	APlayerController* const PC = Cast<APlayerController>(GetController());
	return(PC != nullptr);
}

bool APawn::IsPawnControlled() const
{
	return (GetController() != nullptr);
}

FRotator APawn::GetControlRotation() const
{
	return GetController() ? GetController()->GetControlRotation() : FRotator::ZeroRotator;
}

void APawn::OnRep_Controller()
{
	AController* const ThisController = GetController();
	bool bNotifyControllerChange = UE::Gameplay::CVars::bAlwaysNotifyClientOnControllerChange ?
		(ThisController != PreviousController) :	// By default, notify whenever the PreviousController is out of date for any reason
		(ThisController == nullptr);				// In backward compatibility, only notify when changing from null or the edge case below
	
	if ( (ThisController != nullptr) && (ThisController->GetPawn() == nullptr) )
	{
		// This ensures that AController::OnRep_Pawn is called. Since we cant ensure replication order of APawn::Controller and AController::Pawn,
		// if APawn::Controller is repped first, it will set AController::Pawn locally. When AController::Pawn is repped, the rep value will not
		// be different from the just set local value, and OnRep_Pawn will not be called. This can cause problems if OnRep_Pawn does anything important.
		//
		// It would be better to never ever set replicated properties locally, but this is pretty core in the gameplay framework and I think there are
		// lots of assumptions made in the code base that the Pawn and Controller will always be linked both ways.
		ThisController->SetPawnFromRep(this); 

		APlayerController* const PC = Cast<APlayerController>(ThisController);
		if ( (PC != nullptr) && PC->bAutoManageActiveCameraTarget && (PC->PlayerCameraManager->ViewTarget.Target == GetController()) )
		{
			PC->AutoManageActiveCameraTarget(this);
		}

		bNotifyControllerChange = true;
	}

	if (bNotifyControllerChange)
	{
		NotifyControllerChanged();
	}
}

void APawn::OnRep_PlayerState()
{
	SetPlayerState(GetPlayerState());
}

void APawn::SetPlayerState(APlayerState* NewPlayerState)
{
	APlayerState* OldPlayerState = PlayerState;

	if (PlayerState && PlayerState->GetPawn() == this)
	{
		FSetPlayerStatePawn(PlayerState, nullptr);
	}

	PlayerState = NewPlayerState;

	if (PlayerState)
	{
		FSetPlayerStatePawn(PlayerState, this);
	}

	OnPlayerStateChanged(NewPlayerState, OldPlayerState);
}

void APawn::PossessedBy(AController* NewController)
{
	SetOwner(NewController);
	
	AController* const OldController = GetController();

	SetController(NewController);
	ForceNetUpdate();

	// The owning connection depends on the Controller having the new value.
	UpdateOwningNetConnection();

	if (GetController()->PlayerState != nullptr)
	{
		SetPlayerState(GetController()->PlayerState);
	}

	if (APlayerController* PlayerController = Cast<APlayerController>(GetController()))
	{
		if (GetNetMode() != NM_Standalone)
		{
			SetReplicates(true);
			if (!PlayerController->IsLocalController())
			{
				SetAutonomousProxy(true);
			}
		}
	}
	else
	{
		CopyRemoteRoleFrom(GetDefault<APawn>());
	}

	// dispatch Blueprint event if necessary
	if (OldController != NewController)
	{
		ReceivePossessed(GetController());
	
		NotifyControllerChanged();
	}
}

void APawn::UnPossessed()
{
	AController* const OldController = GetController();

	ForceNetUpdate();

	SetPlayerState(nullptr);
	SetOwner(nullptr);
	SetController(nullptr);

	// The owning connection depends on the Controller having the new value.
	UpdateOwningNetConnection();

	// Unregister input component if we created one
	DestroyPlayerInputComponent();

	// dispatch Blueprint event if necessary
	if (OldController)
	{
		ReceiveUnpossessed(OldController);
	}

	NotifyControllerChanged();

	ConsumeMovementInputVector();
}

void APawn::NotifyControllerChanged()
{
	ReceiveControllerChanged(PreviousController, GetController());
	ReceiveControllerChangedDelegate.Broadcast(this, PreviousController, GetController());
	if (UGameInstance* GameInstance = GetGameInstance())
	{
		GameInstance->GetOnPawnControllerChanged().Broadcast(this, GetController());
	}

	// Update the cached controller
	PreviousController = GetController();
}

class UNetConnection* APawn::GetNetConnection() const
{
	// if have a controller, it has the net connection
	if (GetController())
	{
		return GetController()->GetNetConnection();
	}
	return Super::GetNetConnection();
}

const AActor* APawn::GetNetOwner() const
{
	return this;
}

class UPlayer* APawn::GetNetOwningPlayer()
{
	if (GetLocalRole() == ROLE_Authority)
	{
		if (GetController())
		{
			APlayerController* PC = Cast<APlayerController>(GetController());
			return PC ? PC->Player : nullptr;
		}
	}

	return Super::GetNetOwningPlayer();
}

class UPlayer* APawn::GetNetOwningPlayerAnyRole()
{
	if (GetController())
	{
		APlayerController* PC = Cast<APlayerController>(GetController());
		return PC ? PC->Player : nullptr;
	}

	return Super::GetNetOwningPlayer();
}

UInputComponent* APawn::CreatePlayerInputComponent()
{
	static const FName InputComponentName("PawnInputComponent0");
	const UClass* OverrideClass = OverrideInputComponentClass.Get();
	return NewObject<UInputComponent>(this, OverrideClass ? OverrideClass : UInputSettings::GetDefaultInputComponentClass(), InputComponentName);
}

void APawn::DestroyPlayerInputComponent()
{
	if (InputComponent)
	{
		InputComponent->DestroyComponent();
		InputComponent = nullptr;
	}
}


bool APawn::IsMoveInputIgnored() const
{
	return GetController() != nullptr && GetController()->IsMoveInputIgnored();
}

TSubclassOf<UInputComponent> APawn::GetOverrideInputComponentClass() const
{
	return OverrideInputComponentClass;	
}

void APawn::AddMovementInput(FVector WorldDirection, float ScaleValue, bool bForce /*=false*/)
{
	UPawnMovementComponent* MovementComponent = GetMovementComponent();
	if (MovementComponent)
	{
		MovementComponent->AddInputVector(WorldDirection * ScaleValue, bForce);
	}
	else
	{
		Internal_AddMovementInput(WorldDirection * ScaleValue, bForce);
	}
}


FVector APawn::GetPendingMovementInputVector() const
{
	// There's really no point redirecting to the MovementComponent since GetInputVector is not virtual there, and it just comes back to us.
	return ControlInputVector;
}

FVector APawn::GetLastMovementInputVector() const
{
	return LastControlInputVector;
}

FVector APawn::ConsumeMovementInputVector()
{
	UPawnMovementComponent* MovementComponent = GetMovementComponent();
	if (MovementComponent)
	{
		return MovementComponent->ConsumeInputVector();
	}
	else
	{
		return Internal_ConsumeMovementInputVector();
	}
}


void APawn::Internal_AddMovementInput(FVector WorldAccel, bool bForce /*=false*/)
{
	if (bForce || !IsMoveInputIgnored())
	{
		ControlInputVector += WorldAccel;
	}
}

FVector APawn::Internal_ConsumeMovementInputVector()
{
	LastControlInputVector = ControlInputVector;
	ControlInputVector = FVector::ZeroVector;
	return LastControlInputVector;
}


void APawn::AddControllerPitchInput(float Val)
{
	if (Val != 0.f && GetController() && GetController()->IsLocalPlayerController())
	{
		APlayerController* const PC = CastChecked<APlayerController>(GetController());
		PC->AddPitchInput(Val);
	}
}

void APawn::AddControllerYawInput(float Val)
{
	if (Val != 0.f && GetController() && GetController()->IsLocalPlayerController())
	{
		APlayerController* const PC = CastChecked<APlayerController>(GetController());
		PC->AddYawInput(Val);
	}
}

void APawn::AddControllerRollInput(float Val)
{
	if (Val != 0.f && GetController() && GetController()->IsLocalPlayerController())
	{
		APlayerController* const PC = CastChecked<APlayerController>(GetController());
		PC->AddRollInput(Val);
	}
}


void APawn::Restart()
{
	UPawnMovementComponent* MovementComponent = GetMovementComponent();
	if (MovementComponent)
	{
		MovementComponent->StopMovementImmediately();
	}
	ConsumeMovementInputVector();
	RecalculateBaseEyeHeight();
}

APhysicsVolume* APawn::GetPhysicsVolume() const
{
	if (const UPawnMovementComponent* MovementComponent = GetMovementComponent())
	{
		return MovementComponent->GetPhysicsVolume();
	}

	return Super::GetPhysicsVolume();
}


void APawn::SetPlayerDefaults()
{
}

void APawn::RecalculateBaseEyeHeight()
{
	BaseEyeHeight = GetDefault<APawn>(GetClass())->BaseEyeHeight;
}

void APawn::Reset()
{
	if ( (GetController() == nullptr) || (GetController()->PlayerState != nullptr) )
	{
		DetachFromControllerPendingDestroy();
		Destroy();
	}
	else
	{
		Super::Reset();
	}
}

FString APawn::GetHumanReadableName() const
{
	return GetPlayerState() ? GetPlayerState()->GetPlayerName() : Super::GetHumanReadableName();
}


void APawn::DisplayDebug(class UCanvas* Canvas, const FDebugDisplayInfo& DebugDisplay, float& YL, float& YPos)
{
// 	for (FTouchingComponentsMap::TIterator It(TouchingComponents); It; ++It)
// 	{
// 		UPrimitiveComponent* const MyComp = It.Key().Get();
// 		UPrimitiveComponent* const OtherComp = It.Value().Get();
// 		AActor* OtherActor = OtherComp ? OtherComp->GetOwner() : nullptr;
// 
// 		YL = HUD->Canvas->DrawText(FString::Printf(TEXT("TOUCHING my %s to %s's %s"), *GetNameSafe(MyComp), *GetNameSafe(OtherActor), *GetNameSafe(OtherComp)));
// 		YPos += YL;
// 		HUD->Canvas->SetPos(4,YPos);
// 	}

	FDisplayDebugManager& DisplayDebugManager = Canvas->DisplayDebugManager;
	if (GetPlayerState() == nullptr )
	{
		DisplayDebugManager.DrawString(TEXT("NO PlayerState"));
	}
	else
	{
		GetPlayerState()->DisplayDebug(Canvas, DebugDisplay, YL, YPos);
	}

	Super::DisplayDebug(Canvas, DebugDisplay, YL, YPos);

	DisplayDebugManager.SetDrawColor(FColor(255,255,255));

	if (DebugDisplay.IsDisplayOn(NAME_Camera))
	{
		DisplayDebugManager.DrawString(FString::Printf(TEXT("BaseEyeHeight %f"), BaseEyeHeight));
	}

	// Controller
	if (GetController() == nullptr )
	{
		DisplayDebugManager.SetDrawColor(FColor(255, 0, 0));
		DisplayDebugManager.DrawString(TEXT("NO Controller"));
	}
	else
	{
		GetController()->DisplayDebug(Canvas, DebugDisplay, YL, YPos);
	}
}


void APawn::GetActorEyesViewPoint( FVector& out_Location, FRotator& out_Rotation ) const
{
	out_Location = GetPawnViewLocation();
	out_Rotation = GetViewRotation();
}


FRotator APawn::GetBaseAimRotation() const
{
	// If we have a controller, by default we aim at the player's 'eyes' direction
	// that is by default Controller.Rotation for AI, and camera (crosshair) rotation for human players.
	FVector POVLoc;
	FRotator POVRot;
	if(GetController() != nullptr && !InFreeCam() )
	{
		GetController()->GetPlayerViewPoint(POVLoc, POVRot);
		return POVRot;
	}

	// If we have no controller, we simply use our rotation
	POVRot = GetActorRotation();

	// If our Pitch is 0, then use a replicated view pitch
	if( FMath::IsNearlyZero(POVRot.Pitch) )
	{
		const UWorld* World = GetWorld();
		const UDemoNetDriver* DemoNetDriver = World ? World->GetDemoNetDriver() : nullptr;

		if (DemoNetDriver && DemoNetDriver->IsPlaying() && (DemoNetDriver->GetPlaybackEngineNetworkProtocolVersion() < FEngineNetworkCustomVersion::PawnRemoteViewPitch))
		{
			// Use deprecated 8-bit value serialized from old replays
			PRAGMA_DISABLE_DEPRECATION_WARNINGS
			POVRot.Pitch = RemoteViewPitch;
			PRAGMA_ENABLE_DEPRECATION_WARNINGS
			POVRot.Pitch = POVRot.Pitch * 360.0f / 255.0f;
		}
		else if (DemoNetDriver && DemoNetDriver->IsPlaying() && (DemoNetDriver->GetPlaybackEngineNetworkProtocolVersion() < FEngineNetworkCustomVersion::PawnRemoteViewPitchTo16Bit))
		{
			// Use deprecated 8-bit value serialized from old replays
			PRAGMA_DISABLE_DEPRECATION_WARNINGS
			POVRot.Pitch = FRotator::DecompressAxisFromByte(RemoteViewPitch);
			PRAGMA_ENABLE_DEPRECATION_WARNINGS
		}
		else
		{
			POVRot.Pitch = FRotator::DecompressAxisFromShort(RemoteViewPitch16);
		}
	}

	return POVRot;
}

bool APawn::InFreeCam() const
{
	const APlayerController* PC = Cast<const APlayerController>(GetController());
	static FName NAME_FreeCam =  FName("FreeCam");
	static FName NAME_FreeCamDefault = FName("FreeCam_Default");
	return (PC != nullptr && PC->PlayerCameraManager != nullptr && (PC->PlayerCameraManager->CameraStyle == NAME_FreeCam || PC->PlayerCameraManager->CameraStyle == NAME_FreeCamDefault) );
}

void APawn::OutsideWorldBounds()
{
	if ( !bProcessingOutsideWorldBounds )
	{
		bProcessingOutsideWorldBounds = true;
		// AI pawns on the server just destroy
		if (GetLocalRole() == ROLE_Authority && Cast<APlayerController>(GetController()) == nullptr)
		{
			Destroy();
		}
		else
		{
			DetachFromControllerPendingDestroy();
			TurnOff();
			SetActorHiddenInGame(true);
			SetLifeSpan( FMath::Clamp(InitialLifeSpan, 0.1f, 1.0f) );
		}
		bProcessingOutsideWorldBounds = false;
	}
}

void APawn::FaceRotation(FRotator NewControlRotation, float DeltaTime)
{
	// Only if we actually are going to use any component of rotation.
	if (bUseControllerRotationPitch || bUseControllerRotationYaw || bUseControllerRotationRoll)
	{
		const FRotator CurrentRotation = GetActorRotation();

		if (!bUseControllerRotationPitch)
		{
			NewControlRotation.Pitch = CurrentRotation.Pitch;
		}

		if (!bUseControllerRotationYaw)
		{
			NewControlRotation.Yaw = CurrentRotation.Yaw;
		}

		if (!bUseControllerRotationRoll)
		{
			NewControlRotation.Roll = CurrentRotation.Roll;
		}

#if ENABLE_NAN_DIAGNOSTIC
		if (NewControlRotation.ContainsNaN())
		{
			logOrEnsureNanError(TEXT("APawn::FaceRotation about to apply NaN-containing rotation to actor! New:(%s), Current:(%s)"), *NewControlRotation.ToString(), *CurrentRotation.ToString());
		}
#endif

		SetActorRotation(NewControlRotation);
	}
}

void APawn::DetachFromControllerPendingDestroy()
{
	AController* const ThisController = GetController();
	if (ThisController != nullptr && ThisController->GetPawn() == this )
	{
		ThisController->PawnPendingDestroy(this);
		if (ThisController != nullptr)
		{
			ThisController->UnPossess();
			SetController(nullptr);
		}
	}
}

AController* APawn::GetDamageInstigator(AController* InstigatedBy, const UDamageType& DamageType) const
{
	if ( (InstigatedBy != nullptr) && (InstigatedBy != GetController()) )
	{
		return InstigatedBy;
	}
	else if ( DamageType.bCausedByWorld && (LastHitBy != nullptr) )
	{
		return LastHitBy;
	}
	return InstigatedBy;
}


#if WITH_EDITOR
void APawn::EditorApplyRotation(const FRotator& DeltaRotation, bool bAltDown, bool bShiftDown, bool bCtrlDown)
{
	Super::EditorApplyRotation(DeltaRotation, bAltDown, bShiftDown, bCtrlDown);

	// Forward new rotation on to the pawn's controller.
	if(GetController())
	{
		GetController()->SetControlRotation( GetActorRotation() );
	}
}

void APawn::PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent)
{
	static const FName NAME_CanAffectNavigationGeneration = FName("bCanAffectNavigationGeneration");

	Super::PostEditChangeProperty(PropertyChangedEvent);

	if (PropertyChangedEvent.Property != nullptr && PropertyChangedEvent.Property->GetFName() == NAME_CanAffectNavigationGeneration)
	{
		SetCanAffectNavigationGeneration(bCanAffectNavigationGeneration, /*bForceUpdate=*/true);
	}
}
#endif

void APawn::EnableInput(class APlayerController* PlayerController)
{
	if (PlayerController == GetController() || PlayerController == nullptr)
	{
		bInputEnabled = true;
	}
	else
	{
		UE_LOGF(LogPawn, Error, "EnableInput can only be specified on a Pawn for its Controller");
	}
}

void APawn::DisableInput(class APlayerController* PlayerController)
{
	if (PlayerController == GetController() || PlayerController == nullptr)
	{
		bInputEnabled = false;
	}
	else
	{
		UE_LOGF(LogPawn, Error, "DisableInput can only be specified on a Pawn for its Controller");
	}
}

void APawn::TeleportSucceeded(bool bIsATest)
{
	if (bIsATest == false)
	{
		UPawnMovementComponent* MovementComponent = GetMovementComponent();
		if (MovementComponent)
		{
			MovementComponent->OnTeleported();
		}
	}

	Super::TeleportSucceeded(bIsATest);
}

void APawn::GetMoveGoalReachTest(const AActor* MovingActor, const FVector& MoveOffset, FVector& GoalOffset, float& GoalRadius, float& GoalHalfHeight) const 
{
	GoalOffset = FVector::ZeroVector;
	GetSimpleCollisionCylinder(GoalRadius, GoalHalfHeight);
}

// REPLICATION

void APawn::PostNetReceiveVelocity(const FVector& NewVelocity)
{
	const FPhysicsPredictionSettings& PhysicsPredictionSettings = UPhysicsSettings::Get()->PhysicsPrediction;
	if (PhysicsPredictionSettings.bEnablePhysicsPrediction)
	{
		Super::PostNetReceiveVelocity(NewVelocity);
	}
	if (GetLocalRole() == ROLE_SimulatedProxy)
	{
		UMovementComponent* const MoveComponent = GetMovementComponent();
		if ( MoveComponent )
		{
			MoveComponent->Velocity = NewVelocity;
		}
	}
}

void APawn::PostNetReceiveLocationAndRotation()
{
	const FRepMovement& ConstRepMovement = GetReplicatedMovement();

	// always consider Location as changed if we were spawned this tick as in that case our replicated Location was set as part of spawning, before PreNetReceive()
	if( (FRepMovement::RebaseOntoLocalOrigin(ConstRepMovement.Location, this) == GetActorLocation()
		&& ConstRepMovement.Rotation == GetActorRotation()) && (CreationTime != GetWorld()->TimeSeconds) )
	{
		return;
	}

	if (GetLocalRole() == ROLE_SimulatedProxy)
	{
		// Correction to make sure pawn doesn't penetrate floor after replication rounding
		FRepMovement& MutableRepMovement = GetReplicatedMovement_Mutable();
		MutableRepMovement.Location.Z += 0.01f;

		const FVector OldLocation = GetActorLocation();
		const FQuat OldRotation = GetActorQuat();
		const FVector NewLocation = FRepMovement::RebaseOntoLocalOrigin(MutableRepMovement.Location, this);
		SetActorLocationAndRotation(NewLocation, MutableRepMovement.Rotation, /*bSweep=*/ false);

		INetworkPredictionInterface* PredictionInterface = Cast<INetworkPredictionInterface>(GetMovementComponent());
		if (PredictionInterface)
		{
			PredictionInterface->SmoothCorrection(OldLocation, OldRotation, NewLocation, MutableRepMovement.Rotation.Quaternion());
		}
	}
}

bool APawn::IsBasedOnActor(const AActor* Other) const
{
	const FMovementBaseInterfaceData* MovementBaseInterfaceData = GetMovementBaseInterfaceData();
	if (MovementBaseInterfaceData && MovementBaseInterfaceData->IsValid())
	{
		const AActor* MovementBaseActor = Cast<AActor>(MovementBaseInterfaceData->GetBodyInstanceOwner()->GetSourceObjectOwner());
		if (MovementBaseActor && MovementBaseActor == Other)
		{
			return true;
		}
	}

	return Super::IsBasedOnActor(Other);
}


bool APawn::IsNetRelevantFor(const AActor* RealViewer, const AActor* ViewTarget, const FVector& SrcLocation) const
{
	CA_SUPPRESS(6011);
	if (bAlwaysRelevant || RealViewer == GetController() || IsOwnedBy(ViewTarget) || IsOwnedBy(RealViewer) || this == ViewTarget || ViewTarget == GetInstigator()
		|| IsBasedOnActor(ViewTarget) || (ViewTarget && ViewTarget->IsBasedOnActor(this)))
	{
		return true;
	}
	else if ((IsHidden() || bOnlyRelevantToOwner) && (!GetRootComponent() || !GetRootComponent()->IsCollisionEnabled())) 
	{
		return false;
	}
	else
	{
		const FMovementBaseInterfaceData* MovementBaseInterfaceData = GetMovementBaseInterfaceData();
		if (MovementBaseInterfaceData && MovementBaseInterfaceData->IsValid())
		{
			AActor* BaseActor = Cast<AActor>(MovementBaseInterfaceData->GetMovementBaseObjectOwner());
			if (BaseActor && GetMovementComponent() && ((Cast<const USkeletalMeshComponent>(MovementBaseInterfaceData->GetMovementBaseObject())) || (BaseActor == GetOwner())) )
			{
				return BaseActor->IsNetRelevantFor(RealViewer, ViewTarget, SrcLocation);
			}
		}
	}

	return !GetDefault<AGameNetworkManager>()->bUseDistanceBasedRelevancy ||
			IsWithinNetRelevancyDistance(SrcLocation);
}

void APawn::GetLifetimeReplicatedProps( TArray< FLifetimeProperty > & OutLifetimeProps ) const
{
	Super::GetLifetimeReplicatedProps( OutLifetimeProps );

	DOREPLIFETIME( APawn, PlayerState ); 
	DOREPLIFETIME( APawn, Controller );

	// Deprecated RemoteViewPitch, still loaded for old replays
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	DOREPLIFETIME_CONDITION( APawn, RemoteViewPitch, COND_ReplayOnly );
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	DOREPLIFETIME_CONDITION( APawn, RemoteViewPitch16, 	COND_SkipOwner );
}

void APawn::SetController(AController* InController)
{
	Controller = InController;
}

void APawn::PreReplication( IRepChangedPropertyTracker & ChangedPropertyTracker )
{
	Super::PreReplication( ChangedPropertyTracker );

	if (GetLocalRole() == ROLE_Authority && GetController())
	{
		SetRemoteViewPitch(GetController()->GetControlRotation().Pitch);
	}
}

void APawn::MoveIgnoreActorAdd(AActor* ActorToIgnore)
{
	UPrimitiveComponent* RootPrimitiveComponent = Cast<UPrimitiveComponent>(GetRootComponent());
	if( RootPrimitiveComponent )
	{
		RootPrimitiveComponent->IgnoreActorWhenMoving(ActorToIgnore, true);
	}
}

void APawn::MoveIgnoreActorRemove(AActor* ActorToIgnore)
{
	UPrimitiveComponent* RootPrimitiveComponent = Cast<UPrimitiveComponent>(GetRootComponent());
	if( RootPrimitiveComponent )
	{
		RootPrimitiveComponent->IgnoreActorWhenMoving(ActorToIgnore, false);
	}
}

void APawn::PawnMakeNoise(float Loudness, FVector NoiseLocation, bool bUseNoiseMakerLocation, AActor* NoiseMaker)
{
	if (NoiseMaker == nullptr)
	{
		NoiseMaker = this;
	}
	NoiseMaker->MakeNoise(Loudness, this, bUseNoiseMakerLocation ? NoiseMaker->GetActorLocation() : NoiseLocation);
}

const FNavAgentProperties& APawn::GetNavAgentPropertiesRef() const
{
	INavMovementInterface* NavMovementInterface = FindComponentByInterface<INavMovementInterface>();
	return NavMovementInterface ? NavMovementInterface->GetNavAgentPropertiesRef() : FNavAgentProperties::DefaultProperties;
}

