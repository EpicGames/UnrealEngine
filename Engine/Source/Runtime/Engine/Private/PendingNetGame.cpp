// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
  PendingNetGame.cpp: Unreal pending net game class.
=============================================================================*/

#include "Engine/PendingNetGame.h"
#include "Engine/GameInstance.h"
#include "EngineGlobals.h"
#include "Engine/Engine.h"
#include "GameFramework/PlayerController.h"
#include "Engine/LocalPlayer.h"
#include "Net/NetworkProfiler.h"
#include "Net/DataChannel.h"
#include "PacketHandler.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PendingNetGame)

void UPendingNetGame::Initialize(const FURL& InURL)
{
	NetDriver = NULL;
	URL = InURL;
	bSuccessfullyConnected = false;
	bSentJoinRequest = false;
	bLoadedMapSuccessfully = false;
	bFailedTravel = false;
}

UPendingNetGame::UPendingNetGame(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

void UPendingNetGame::InitNetDriver(UNetDriver* InNetDriver)
{
	LLM_SCOPE(ELLMTag::Networking);

	if (!GDisallowNetworkTravel)
	{
		NETWORK_PROFILER(GNetworkProfiler.TrackSessionChange(true, URL));

		// Try to create network driver.
		if (InNetDriver == nullptr)
		{
			if (GEngine->CreateNamedNetDriver(this, NAME_PendingNetDriver, NAME_GameNetDriver))
			{
				NetDriver = GEngine->FindNamedNetDriver(this, NAME_PendingNetDriver);
			}
		}
		else
		{
			ensureMsgf(InNetDriver->ServerConnection == nullptr, TEXT("UPendingNetGame: Provided NetDriver is already connected/connecting to a remote server."));
			ensureMsgf(!InNetDriver->ConnectionlessHandler.IsValid(), TEXT("UPendingNetGame: Provided NetDriver is already listening for incoming connections."));
			NetDriver = InNetDriver;
		}

		if (NetDriver == nullptr)
		{
			UE_LOGF(LogNet, Warning, "Error initializing the pending net driver.  Check the configuration of NetDriverDefinitions and make sure module/plugin dependencies are correct.");
			ConnectionError = NSLOCTEXT("Engine", "NetworkDriverInit", "Error creating network driver.").ToString();
			return;
		}

		if( NetDriver->InitConnect( this, URL, ConnectionError ) )
		{
			FNetDelegates::OnPendingNetGameConnectionCreated.Broadcast(this);

			ULocalPlayer* LocalPlayer = GetFirstGamePlayer();
			if (LocalPlayer)
			{
				LocalPlayer->PreBeginHandshake(ULocalPlayer::FOnPreBeginHandshakeCompleteDelegate::CreateWeakLambda(this,
					[this]()
					{
						BeginHandshake();
					}));
			}
			else
			{
				BeginHandshake();
			}
		}
		else
		{
			// error initializing the network stack...
			UE_LOGF(LogNet, Warning, "error initializing the network stack");
			GEngine->DestroyNamedNetDriver(this, NetDriver->NetDriverName);
			NetDriver = NULL;

			// ConnectionError should be set by calling InitConnect...however, if we set NetDriver to NULL without setting a
			// value for ConnectionError, we'll trigger the assertion at the top of UPendingNetGame::Tick() so make sure it's set
			if ( ConnectionError.Len() == 0 )
			{
				ConnectionError = NSLOCTEXT("Engine", "NetworkInit", "Error initializing network layer.").ToString();
			}
		}
	}
	else
	{
		ConnectionError = NSLOCTEXT("Engine", "UsedCheatCommands", "Console commands were used which are disallowed in netplay.  You must restart the game to create a match.").ToString();
	}
}

void UPendingNetGame::FinishDestroy()
{
	ensureMsgf(NetDriver == nullptr, TEXT("PendingNetGame's NetDriver was not shutdown before destruction."));
	
	NetDriver = nullptr;

	Super::FinishDestroy();
}

void UPendingNetGame::BeginHandshake()
{
	// Kick off the connection handshake
	UNetConnection* ServerConn = NetDriver->ServerConnection;
	if (ServerConn->Handler.IsValid())
	{
		ServerConn->Handler->BeginHandshaking(
			FPacketHandlerHandshakeComplete::CreateUObject(this, &UPendingNetGame::SendInitialJoin));
	}
	else
	{
		SendInitialJoin();
	}
}

void UPendingNetGame::SendInitialJoin()
{
	if (NetDriver != nullptr)
	{
		UNetConnection* ServerConn = NetDriver->ServerConnection;

		if (ServerConn != nullptr)
		{
			uint8 IsLittleEndian = uint8(PLATFORM_LITTLE_ENDIAN);
			check(IsLittleEndian == !!IsLittleEndian); // should only be one or zero

			const int32 AllowEncryption = CVarNetAllowEncryption.GetValueOnGameThread();
			FString EncryptionToken;

			if (AllowEncryption != 0)
			{
				EncryptionToken = URL.GetOption(TEXT("EncryptionToken="), TEXT(""));
			}

			bool bEncryptionRequirementsFailure = false;

			if (EncryptionToken.IsEmpty())
			{
				EEncryptionFailureAction FailureResult = EEncryptionFailureAction::Default;

				if (FNetDelegates::OnReceivedNetworkEncryptionFailure.IsBound())
				{
					FailureResult = FNetDelegates::OnReceivedNetworkEncryptionFailure.Execute(ServerConn);
				}

				const bool bGameplayDisableEncryptionCheck = FailureResult == EEncryptionFailureAction::AllowConnection;

				bEncryptionRequirementsFailure = NetDriver->IsEncryptionRequired() && !bGameplayDisableEncryptionCheck;
			}
			
			if (!bEncryptionRequirementsFailure)
			{
				uint32 LocalNetworkVersion = FNetworkVersion::GetLocalNetworkVersion();

				UE_LOGF(LogNet, Log, "UPendingNetGame::SendInitialJoin: Sending hello. %ls", *ServerConn->Describe());

				EEngineNetworkRuntimeFeatures LocalNetworkFeatures = NetDriver->GetNetworkRuntimeFeatures();
				FNetControlMessage<NMT_Hello>::Send(ServerConn, IsLittleEndian, LocalNetworkVersion, EncryptionToken, LocalNetworkFeatures);


				ServerConn->FlushNet();
			}
			else
			{
				UE_LOGF(LogNet, Error, "UPendingNetGame::SendInitialJoin: EncryptionToken is empty when 'net.AllowEncryption' requires it.");

				ConnectionError = TEXT("EncryptionToken not set.");
			}
		}
	}
}

void UPendingNetGame::Serialize( FArchive& Ar )
{
	Super::Serialize(Ar);
	if( !Ar.IsLoading() && !Ar.IsSaving() )
	{
		Ar << NetDriver;
	}
}

void UPendingNetGame::AddReferencedObjects(UObject* InThis, FReferenceCollector& Collector)
{	
	UPendingNetGame* This = CastChecked<UPendingNetGame>(InThis);
#if WITH_EDITOR
	if( GIsEditor )
	{
		// Required by the unified GC when running in the editor
		Collector.AddReferencedObject( This->NetDriver, This );
	}
#endif
	Super::AddReferencedObjects( This, Collector );
}

bool UPendingNetGame::LoadMapCompleted(UEngine* Engine, FWorldContext& Context, bool bInLoadedMapSuccessfully, const FString& LoadMapError)
{
	bLoadedMapSuccessfully = bInLoadedMapSuccessfully;
	if (!bLoadedMapSuccessfully || LoadMapError != TEXT(""))
	{
		// this is handled in the TickWorldTravel
		return false;
	}
	return true;
}

void UPendingNetGame::TravelCompleted(UEngine* Engine, FWorldContext& Context)
{
	// Show connecting message, cause precaching to occur.
	Engine->TransitionType = ETransitionType::Connecting;

	Engine->RedrawViewports(false);

	// Send join.
	Context.PendingNetGame->SendJoin();
	Context.PendingNetGame->NetDriver = NULL;

	UE_LOGSTATUS(Log, TEXT("Pending net game travel completed"));
}

EAcceptConnection::Type UPendingNetGame::NotifyAcceptingConnection()
{
	return EAcceptConnection::Reject; 
}
void UPendingNetGame::NotifyAcceptedConnection( class UNetConnection* Connection )
{
}
bool UPendingNetGame::NotifyAcceptingChannel( class UChannel* Channel )
{
	return false;
}

void UPendingNetGame::NotifyControlMessage(UNetConnection* Connection, uint8 MessageType, class FInBunch& Bunch)
{
	check(Connection == NetDriver->ServerConnection);

#if !UE_BUILD_SHIPPING
	UE_LOGF(LogNet, Verbose, "NotifyControlMessage: PendingLevel received: %ls", FNetControlMessageInfo::GetName(MessageType));
#endif

	// This client got a response from the server.
	switch (MessageType)
	{
		case NMT_Upgrade:
		{
			// Report mismatch.
			uint32 RemoteNetworkVersion = 0;
			EEngineNetworkRuntimeFeatures RemoteNetworkFeatures = EEngineNetworkRuntimeFeatures::None;

			if (FNetControlMessage<NMT_Upgrade>::Receive(Bunch, RemoteNetworkVersion, RemoteNetworkFeatures))
			{
				const bool bUpgradeSuccess = Connection->HandleReceiveNetUpgrade(RemoteNetworkVersion, RemoteNetworkFeatures);
				if (bUpgradeSuccess)
				{
					// Restart the handshake process now that we are compatible
					SendInitialJoin();
				}
				else
				{
					// Upgrade failed, trigger a disconnect via the connect error
					ConnectionError = NSLOCTEXT("Engine", "ClientOutdated", "The match you are trying to join is running an incompatible version of the game.  Please try upgrading your game version.").ToString();
				}
			}

			break;
		}
		case NMT_Failure:
		{
			// our connection attempt failed for some reason, for example a synchronization mismatch (bad GUID, etc) or because the server rejected our join attempt (too many players, etc)
			// here we can further parse the string to determine the reason that the server closed our connection and present it to the user

			FString ErrorMsg;

			if (FNetControlMessage<NMT_Failure>::Receive(Bunch, ErrorMsg))
			{
				if (ErrorMsg.IsEmpty())
				{
					ErrorMsg = NSLOCTEXT("NetworkErrors", "GenericPendingConnectionFailed", "Pending Connection Failed.").ToString();
				}

				// This error will be resolved in TickWorldTravel()
				ConnectionError = ErrorMsg;

				// Force close the session
				UE_LOGF(LogNet, Log, "NetConnection::Close() [%ls] [%ls] [%ls] from NMT_Failure %ls",
					Connection->Driver ? *Connection->Driver->NetDriverName.ToString() : TEXT("NULL"),
					Connection->PlayerController ? *Connection->PlayerController->GetName() : TEXT("NoPC"),
					Connection->OwningActor ? *Connection->OwningActor->GetName() : TEXT("No Owner"),
					*ConnectionError);

				Connection->Close(ENetCloseResult::FailureReceived);
			}

			break;
		}
		case NMT_Challenge:
		{
			// Challenged by server.
			if (FNetControlMessage<NMT_Challenge>::Receive(Bunch, Connection->Challenge))
			{
				FURL PartialURL(URL);
				PartialURL.Host = TEXT("");
				PartialURL.Port = PartialURL.UrlConfig.DefaultPort; // HACK: Need to fix URL parsing 
				PartialURL.Map = TEXT("");

				for (int32 i = URL.Op.Num() - 1; i >= 0; i--)
				{
					if (URL.Op[i].Left(5) == TEXT("game="))
					{
						URL.Op.RemoveAt(i);
					}
				}

				ULocalPlayer* LocalPlayer = GetFirstGamePlayer();
				if (LocalPlayer)
				{
					// Send the player nickname if available
					FString OverrideName = LocalPlayer->GetNickname();
					if (OverrideName.Len() > 0)
					{
						PartialURL.AddOption(*FString::Printf(TEXT("Name=%s"), *OverrideName));
					}

					// Send any game-specific url options for this player
					FString GameUrlOptions = LocalPlayer->GetGameLoginOptions();
					if (GameUrlOptions.Len() > 0)
					{
						PartialURL.AddOption(*FString::Printf(TEXT("%s"), *GameUrlOptions));
					}

					// Send the player unique Id at login
					Connection->PlayerId = LocalPlayer->GetPreferredUniqueNetId();
				}

				// Send the player's online platform name
				FName OnlinePlatformName = NAME_None;
				if (const FWorldContext* const WorldContext = GEngine->GetWorldContextFromPendingNetGame(this))
				{
					if (WorldContext->OwningGameInstance)
					{
						OnlinePlatformName = WorldContext->OwningGameInstance->GetOnlinePlatformName();
					}
				}

				Connection->ClientResponse = TEXT("0");
				FString URLString(PartialURL.ToString());
				FString OnlinePlatformNameString = OnlinePlatformName.ToString();

				FNetControlMessage<NMT_Login>::Send(Connection, Connection->ClientResponse, URLString, Connection->PlayerId, OnlinePlatformNameString);
				NetDriver->ServerConnection->FlushNet();
			}
			else
			{
				Connection->Challenge.Empty();
			}

			break;
		}
		case NMT_Welcome:
		{
			// Server accepted connection.
			FString GameName;
			FString RedirectURL;

			if (FNetControlMessage<NMT_Welcome>::Receive(Bunch, URL.Map, GameName, RedirectURL))
			{
				//GEngine->NetworkRemapPath(this, URL.Map);

				UE_LOGF(LogNet, Log, "Welcomed by server (Level: %ls, Game: %ls)", *URL.Map, *GameName);

				// extract map name and options
				{
					FURL DefaultURL;
					FURL TempURL(&DefaultURL, *URL.Map, TRAVEL_Partial);
					URL.Map = TempURL.Map;
					URL.RedirectURL = RedirectURL;
					URL.Op.Append(TempURL.Op);
				}

				if (GameName.Len() > 0)
				{
					URL.AddOption(*FString::Printf(TEXT("game=%s"), *GameName));
				}

				// Send out netspeed now that we're connected
				FNetControlMessage<NMT_Netspeed>::Send(Connection, Connection->CurrentNetSpeed);

				// We have successfully connected
				// TickWorldTravel will load the map and call LoadMapCompleted which eventually calls SendJoin
				bSuccessfullyConnected = true;
			}
			else
			{
				URL.Map.Empty();
			}

			break;
		}
		case NMT_NetGUIDAssign:
		{
			FNetworkGUID NetGUID;
			FString Path;

			if (FNetControlMessage<NMT_NetGUIDAssign>::Receive(Bunch, NetGUID, Path))
			{
				NetDriver->ServerConnection->PackageMap->ResolvePathAndAssignNetGUID(NetGUID, Path);
			}

			break;
		}
		case NMT_EncryptionAck:
		{
			if (FNetDelegates::OnReceivedNetworkEncryptionAck.IsBound())
			{
				TWeakObjectPtr<UNetConnection> WeakConnection = Connection;
				FNetDelegates::OnReceivedNetworkEncryptionAck.Execute(FOnEncryptionKeyResponse::CreateUObject(this, &UPendingNetGame::FinalizeEncryptedConnection, WeakConnection));
			}
			else
			{
				// This error will be resolved in TickWorldTravel()
				ConnectionError = TEXT("No encryption ack handler");

				// Force close the session
				UE_LOGF(LogNet, Warning, "%ls: No delegate available to handle encryption ack, disconnecting.", *Connection->GetName());
				Connection->Close();
			}
			break;
		}
		default:
			UE_LOGF(LogNet, Log, " --- Unknown/unexpected message for pending level");
			break;
	}
}

void UPendingNetGame::FinalizeEncryptedConnection(const FEncryptionKeyResponse& Response, TWeakObjectPtr<UNetConnection> WeakConnection)
{
	UNetConnection* Connection = WeakConnection.Get();
	if (Connection)
	{
		if (Connection->GetConnectionState() != USOCK_Invalid && Connection->GetConnectionState() != USOCK_Closed && Connection->Driver)
		{
			if (Response.Response == EEncryptionResponse::Success)
			{
				Connection->EnableEncryption(Response.EncryptionData);
			}
			else
			{
				// This error will be resolved in TickWorldTravel()
				FString ResponseStr(LexToString(Response.Response));
				UE_LOGF(LogNet, Warning, "UPendingNetGame::FinalizeEncryptedConnection: encryption failure [%ls] %ls", *ResponseStr, *Response.ErrorMsg);
				ConnectionError = TEXT("Encryption ack failure");
				Connection->Close();
			}
		}
		else
		{
			// This error will be resolved in TickWorldTravel()
			UE_LOGF(LogNet, Warning, "UPendingNetGame::FinalizeEncryptedConnection: connection in invalid state. %ls", *Connection->Describe());
			ConnectionError = TEXT("Connection encryption state failure");
			Connection->Close();
		}
	}
	else
	{
		// This error will be resolved in TickWorldTravel()
		UE_LOGF(LogNet, Warning, "UPendingNetGame::FinalizeEncryptedConnection: Connection is null.");
		ConnectionError = TEXT("Connection missing during encryption ack");
	}
}

void UPendingNetGame::SetEncryptionKey(const FEncryptionKeyResponse& Response)
{
	if (CVarNetAllowEncryption.GetValueOnGameThread() == 0)
	{
		UE_LOGF(LogNet, Log, "UPendingNetGame::SetEncryptionKey: net.AllowEncryption is false.");
		return;
	}

	if (NetDriver)
	{
		UNetConnection* const Connection = NetDriver->ServerConnection;
		if (Connection)
		{
			if (Connection->GetConnectionState() != USOCK_Invalid && Connection->GetConnectionState() != USOCK_Closed && Connection->Driver)
			{
				if (Response.Response == EEncryptionResponse::Success)
				{
					Connection->SetEncryptionData(Response.EncryptionData);
				}
				else
				{
					// This error will be resolved in TickWorldTravel()
					FString ResponseStr(LexToString(Response.Response));
					UE_LOGF(LogNet, Warning, "UPendingNetGame::SetEncryptionKey: encryption failure [%ls] %ls", *ResponseStr, *Response.ErrorMsg);
					ConnectionError = TEXT("Encryption failure");
					Connection->Close();
				}
			}
			else
			{
				// This error will be resolved in TickWorldTravel()
				UE_LOGF(LogNet, Warning, "UPendingNetGame::SetEncryptionKey: connection in invalid state. %ls", *Connection->Describe());
				ConnectionError = TEXT("Connection encryption state failure");
				Connection->Close();
			}
		}
		else
		{
			// This error will be resolved in TickWorldTravel()
			UE_LOGF(LogNet, Warning, "UPendingNetGame::SetEncryptionKey: Connection is null.");
			ConnectionError = TEXT("Connection missing during encryption ack");
		}
	}
	else
	{
		// This error will be resolved in TickWorldTravel()
		UE_LOGF(LogNet, Warning, "UPendingNetGame::SetEncryptionKey: NetDriver is null.");
		ConnectionError = TEXT("NetDriver missing during encryption ack");
	}
}

void UPendingNetGame::Tick( float DeltaTime )
{
	check(NetDriver && NetDriver->ServerConnection);

	// Handle timed out or failed connection.
	if (NetDriver->ServerConnection->GetConnectionState() == USOCK_Closed && ConnectionError == TEXT(""))
	{
		ConnectionError = NSLOCTEXT("Engine", "ConnectionFailed", "Your connection to the host has been lost.").ToString();
		return;
	}

	/**
	 *   Update the network driver
	 *   ****may NULL itself via CancelPending if a disconnect/error occurs****
	 */
	NetDriver->TickDispatch(DeltaTime);

	if (NetDriver)
	{
		NetDriver->PostTickDispatch();
	}

	if (NetDriver)
	{
		NetDriver->TickFlush(DeltaTime);
	}

	if (NetDriver)
	{
		NetDriver->PostTickFlush();
	}
}

void UPendingNetGame::SendJoin()
{
	SendJoinWithFlags(UE::Net::EJoinFlags::None);
}

void UPendingNetGame::SendJoinWithFlags(UE::Net::EJoinFlags Flags)
{
	bSentJoinRequest = true;

	if (EnumHasAllFlags(Flags, UE::Net::EJoinFlags::NoPawn))
	{
		FNetControlMessage<NMT_JoinNoPawn>::Send(NetDriver->ServerConnection);
		NetDriver->ServerConnection->FlushNet(true);
	}
	else
	{
		FNetControlMessage<NMT_Join>::Send(NetDriver->ServerConnection);
		NetDriver->ServerConnection->FlushNet(true);
	}
}

ULocalPlayer* UPendingNetGame::GetFirstGamePlayer()
{
	return GEngine->GetFirstGamePlayer(this);
}