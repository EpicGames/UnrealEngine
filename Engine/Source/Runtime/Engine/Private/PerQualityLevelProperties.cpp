// Copyright Epic Games, Inc. All Rights Reserved.
#include "PerQualityLevelProperties.h"
#include "Misc/ConfigCacheIni.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PerQualityLevelProperties)

#if WITH_EDITOR
#include "Interfaces/ITargetPlatform.h"
#include "PlatformInfo.h"
#include "DeviceProfiles/DeviceProfile.h"
#include "DeviceProfiles/DeviceProfileManager.h"
#include "UObject/ObjectSaveContext.h"
#endif


namespace QualityLevelProperty
{
	static TArray<FName> QualityLevelNames = { FName("Low"), FName("Medium"), FName("High"), FName("Epic"), FName("Cinematic") };
	static FString QualityLevelMappingStr = "QualityLevelMapping";

	template<typename _ValueType>
	TMap<int32, _ValueType> ConvertQualityLevelData(const TMap<EPerQualityLevels, _ValueType>& Data)
	{
		TMap<int32, _ValueType> ConvertedData;

		for (const TPair<EPerQualityLevels, int32>& Pair : Data)
		{
			ConvertedData.Add((int32)Pair.Key,Pair.Value);
		}

		return ConvertedData;
	}

	template<typename _ValueType>
	TMap<EPerQualityLevels, _ValueType> ConvertQualityLevelData(const TMap<int32, _ValueType>& Data)
	{
		TMap<EPerQualityLevels, _ValueType> ConvertedData;

		for (const TPair<int32, _ValueType>& Pair : Data)
		{
			ConvertedData.Add((EPerQualityLevels)Pair.Key, Pair.Value);
		}

		return ConvertedData;
	}

	FName QualityLevelToFName(int32 QL)
	{
		if (QL >= 0 && QL < static_cast<int32>(EPerQualityLevels::Num))
		{
			return QualityLevelNames[QL];
		}
		else
		{
			return NAME_None;
		}
	}

	int32 FNameToQualityLevel(FName QL)
	{
		return QualityLevelNames.IndexOfByKey(QL);
	}

#if WITH_EDITOR
	static TMap<FString, FSupportedQualityLevelArray> CachedPerPlatformToQualityLevels;
	static FCriticalSection MappingCriticalSection;
	TArray<FName> GetEnginePlatformsForPlatformOrGroupName(const FString& InPlatformName)
	{
		// All EnginePlatforms that correspond with this name.
		TArray<FName> EnginePlatforms;

		FName PlatformName(*InPlatformName); // This may change to match the IniPlatformName

		// Find all platforms for which this is the group name.
		bool bIsGroupName = false;
		for (const FDataDrivenPlatformInfo* DataDrivenPlatformInfo : FDataDrivenPlatformInfoRegistry::GetSortedPlatformInfos(EPlatformInfoType::TruePlatformsOnly))
		{
			// gather all platform related to the platform group
			if (DataDrivenPlatformInfo->PlatformGroupName == PlatformName)
			{
				EnginePlatforms.AddUnique(DataDrivenPlatformInfo->IniPlatformName);
				bIsGroupName = true;
			}
		}
		if (!bIsGroupName)
		{
			const FName& IniPlatformName = FDataDrivenPlatformInfoRegistry::GetPlatformInfo(PlatformName).IniPlatformName;
			if (!IniPlatformName.IsNone())
			{
				if (PlatformName != IniPlatformName)
				{
					PlatformName = IniPlatformName;
				}
				EnginePlatforms.Add(PlatformName);
			}
		}

		return EnginePlatforms;
	}

	FSupportedQualityLevelArray PerPlatformOverrideMapping(FString& InPlatformName, UObject* RequestingAsset)
	{
		FSupportedQualityLevelArray* CachedMappingQualitLevelInfo = nullptr;
		{
			FScopeLock ScopeLock(&MappingCriticalSection);
			CachedMappingQualitLevelInfo = CachedPerPlatformToQualityLevels.Find(InPlatformName);
			if (CachedMappingQualitLevelInfo)
			{
				return *CachedMappingQualitLevelInfo;
			}
		}

		// Platform (group) names
		const TArray<FName>& PlatformGroupNameArray = PlatformInfo::GetAllPlatformGroupNames();
		TArray<FName> EnginePlatforms;

		bool bIsPlatformGroup = PlatformGroupNameArray.Contains(FName(*InPlatformName));
		if (bIsPlatformGroup)
		{
			// get all the platforms from that group
			for (const FDataDrivenPlatformInfo* DataDrivenPlatformInfo : FDataDrivenPlatformInfoRegistry::GetSortedPlatformInfos(EPlatformInfoType::TruePlatformsOnly))
			{
				// gather all platform related to the platform group
				if (DataDrivenPlatformInfo->PlatformGroupName == FName(*InPlatformName))
				{
					EnginePlatforms.AddUnique(DataDrivenPlatformInfo->IniPlatformName);
				}
			}
		}
		else
		{
			FName PlatformFName = FDataDrivenPlatformInfoRegistry::GetPlatformInfo(FName(InPlatformName)).IniPlatformName;
			if (!PlatformFName.IsNone())
			{
				InPlatformName = PlatformFName.ToString();
			}

			EnginePlatforms.AddUnique(FName(*InPlatformName));
		}

		FSupportedQualityLevelArray QualityLevels;

		for (const FName& EnginePlatformName : EnginePlatforms)
		{
			//load individual platform ini files
			FConfigFile EngineSettings;
			 FConfigCacheIni::LoadLocalIniFile(EngineSettings, TEXT("Engine"), true, *EnginePlatformName.ToString());

			FString MappingStr;
			if (EngineSettings.GetString(TEXT("SystemSettings"), *QualityLevelMappingStr, MappingStr))
			{
				int32 Value = FNameToQualityLevel(FName(*MappingStr));
				if (Value == INDEX_NONE)
				{
					UE_LOGF(LogCore, Warning, "Bad QualityLevelMapping input value in %lsEngine.ini. "
						"Needs to be one of [low,medium,high,epic,cinematic].",
						*EnginePlatformName.ToString());
					continue;
				}
				QualityLevels.Add(Value);
			}
			else
			{
				UE_LOGF(LogCore, Warning, "Didn't find QualityLevelMapping in %lsEngine.ini, for platform %ls that was requested by %ls. "
					"Need to define QualityLevelMapping under the [SystemSettings] section. All perplatform MinLOD will not be converted to PerQuality.",
					*EnginePlatformName.ToString(), *EnginePlatformName.ToString(),
					(RequestingAsset ? *RequestingAsset->GetPathName() : TEXT("<Unknown>")));
			}
		}

		// Cache the Scalability setting for this platform
		{
			FScopeLock ScopeLock(&MappingCriticalSection);
			CachedMappingQualitLevelInfo = &CachedPerPlatformToQualityLevels.Add(InPlatformName, QualityLevels);
			return *CachedMappingQualitLevelInfo;
		}
	}

	template<typename StructType, typename ValueType>
	void SaveQualityLevel(const StructType& Property, FSavedData<ValueType>& OutSavedData)
	{
		OutSavedData.Default = Property.Default;
		OutSavedData.PerQuality = Property.PerQuality;
	}

	template<typename StructType, typename ValueType>
	void RestoreQualityLevel(const FSavedData<ValueType>& SavedProperty, StructType& OutProperty)
	{
		OutProperty.Default = SavedProperty.Default;
		OutProperty.PerQuality = SavedProperty.PerQuality;
	}
#endif
}

#if WITH_EDITOR
static TMap<FString, FSupportedQualityLevelArray> GSupportedQualityLevels;
static FCriticalSection GCookCriticalSection;

template<typename StructType, typename ValueType, EName _BasePropertyName>
void FPerQualityLevelProperty<StructType, ValueType, _BasePropertyName>::ConvertQualityLevelData(const TMap<FName, ValueType>& PlaformData, const TMultiMap<FName, FName>& PerPlatformToQualityLevel, ValueType Default)
{
	StructType* This = StaticCast<StructType*>(this);
	
	This->Default = Default;

	// convert each platform overrides
	for (const TPair<FName, ValueType>& Pair : PlaformData)
	{
		// get all quality levels associated with the PerPlatform override 
		TArray<FName> QLNames;
		PerPlatformToQualityLevel.MultiFind(Pair.Key, QLNames);

		for (const FName& QLName : QLNames)
		{
			int32 QLKey = QualityLevelProperty::FNameToQualityLevel(QLName);
			if (QLKey != INDEX_NONE)
			{
				ValueType* Value = This->PerQuality.Find(QLKey);

				// if the quality level already as a value, only change it if the value is lower
				// this can happen if two mapping key as the same quality level but different values
				if (Value != nullptr && Pair.Value < *Value)
				{
					// only change the override if its a smaller 
					*Value = Pair.Value;
				}
				else
				{
					This->PerQuality.Add(QLKey, Pair.Value);
				}
			}
		}
	}
}

template<typename StructType, typename ValueType, EName _BasePropertyName>
int32 FPerQualityLevelProperty<StructType, ValueType, _BasePropertyName>::GetValueForPlatform(const ITargetPlatform* TargetPlatform) const
{
	const StructType* This = StaticCast<const StructType*>(this);
	// get all supported quality level from scalability + engine ini files
	FSupportedQualityLevelArray SupportedQualityLevels = GetSupportedQualityLevels(*TargetPlatform->GetPlatformInfo().IniPlatformName.ToString());

	// loop through all the supported quality level to find the min value
	ValueType MinValue = This->MaxType();

	for (int32& QL : SupportedQualityLevels)
	{
		// check if have data for the supported quality level
		if (IsQualityLevelValid(QL))
		{
			MinValue = FMath::Min(GetValueForQualityLevel(QL), MinValue);
		}
	}

	if (MinValue == This->MaxType())
	{
		MinValue = This->Default;
	}

	return MinValue;
}

template<typename StructType, typename ValueType, EName _BasePropertyName>
FSupportedQualityLevelArray FPerQualityLevelProperty<StructType, ValueType, _BasePropertyName>::GetSupportedQualityLevels(const TCHAR* InPlatformName) const
{
	const FString PlatformNameStr = FDataDrivenPlatformInfoRegistry::GetPlatformInfo(FName(InPlatformName)).IniPlatformName.ToString();
	InPlatformName = *PlatformNameStr;

	FSupportedQualityLevelArray* CachedCookingQualitLevelInfo = nullptr;
	FString UniqueName = FString(InPlatformName) + CVarName;
	{
		FScopeLock ScopeLock(&GCookCriticalSection);
		CachedCookingQualitLevelInfo = GSupportedQualityLevels.Find(UniqueName);
		if (CachedCookingQualitLevelInfo)
		{
			return *CachedCookingQualitLevelInfo;
		}
	}

	FSupportedQualityLevelArray CookingQualitLevelInfo;

	//get the platform config cache ini
	FConfigCacheIni* ConfigSystemPlatform = FConfigCacheIni::ForPlatform(FName(InPlatformName));

	// check the Engine file
	if (FConfigFile* PlatformEngine = ConfigSystemPlatform->FindConfigFile(GEngineIni))
	{
		int32 PropertyQualityLevel = -1;
		if (PlatformEngine->GetInt(TEXT("SystemSettings"), *CVarName, PropertyQualityLevel))
		{
			CookingQualitLevelInfo.Add(PropertyQualityLevel);
		}
	}

	// Load the scalability platform file
	if (FConfigFile* PlatformScalability = ConfigSystemPlatform->FindConfigFile(GScalabilityIni))
	{
		//check all possible quality levels specify in the scalability ini 
		for (int32 QualityLevel = 0; QualityLevel < (int32)EPerQualityLevels::Num; ++QualityLevel)
		{
			FString QualitLevelSectionName = Scalability::GetScalabilitySectionString(*ScalabilitySection, QualityLevel, (int32)EPerQualityLevels::Num);
			int32 PropertyQualityLevel = -1;
			PlatformScalability->GetInt(*QualitLevelSectionName, *CVarName, PropertyQualityLevel);

			// add supported quality level to the property map
			if (PropertyQualityLevel != -1)
			{
				CookingQualitLevelInfo.Add(PropertyQualityLevel);
			}
		}
	}

	// get all the device profile related to the platform we want to cook
	TArray<TObjectPtr<UDeviceProfile>> DeviceProfiles = UDeviceProfileManager::Get().Profiles.FilterByPredicate([&InPlatformName](const TObjectPtr<UDeviceProfile>& DeviceProfile)
	{
		return (DeviceProfile->DeviceType == InPlatformName);
	});

	const FName CVarFName(CVarName);
	// iterate through the DP sections to find the ones relevant to the platform
	for (TObjectPtr<UDeviceProfile>& DeviceProfile : DeviceProfiles)
	{
		const TMap<FName, TSet<FString>> AllPossibleCVars = UDeviceProfileManager::GetAllReferencedDeviceProfileCVars(DeviceProfile);
		const TSet<FString>* CVarValues = AllPossibleCVars.Find(CVarFName);
		if (CVarValues)
		{
			for(const FString& CVarValue : *CVarValues)
			{
				int32 Result = FCString::Atoi(*CVarValue);
				CookingQualitLevelInfo.Add(Result);
			}
		}
	}

	// Cache the Scalability setting for this platform
	{
		FScopeLock ScopeLock(&GCookCriticalSection);
		CachedCookingQualitLevelInfo = &GSupportedQualityLevels.Add(UniqueName, CookingQualitLevelInfo);
		return *CachedCookingQualitLevelInfo;
	}
}

template<typename StructType, typename ValueType, EName _BasePropertyName>
void FPerQualityLevelProperty<StructType, ValueType, _BasePropertyName>::StripQualityLevelForCooking(const TCHAR* InPlatformName)
{
	StructType* This = StaticCast<StructType*>(this);
	if (This->PerQuality.Num() > 0 && !CVarName.IsEmpty())
	{
		FSupportedQualityLevelArray CookQualityLevelInfo = This->GetSupportedQualityLevels(InPlatformName);
		CookQualityLevelInfo.Sort([&](const int32& A, const int32& B) { return (A > B); });

		// remove unsupported quality levels 
		for (typename TMap<int32, ValueType>::TIterator It(This->PerQuality); It; ++It)
		{
			if (!CookQualityLevelInfo.Contains(It.Key()))
			{
				It.RemoveCurrent();
			}
		}

		if (This->PerQuality.Num() > 0)
		{
			ValueType PreviousQualityLevel = This->Default;

			for (TSet<int32>::TIterator It(CookQualityLevelInfo); It; ++It)
			{
				ValueType* QualityLevelMinValue = This->PerQuality.Find(*It);
			
				// add quality level supported by the platform
				if (!QualityLevelMinValue)
				{
					This->PerQuality.Add(*It, PreviousQualityLevel);
				}
				else
				{
					PreviousQualityLevel = *QualityLevelMinValue;
				}
			}

			This->Default = PreviousQualityLevel;
		}
	}
}

template<typename StructType, typename ValueType, EName _BasePropertyName>
bool FPerQualityLevelProperty<StructType, ValueType, _BasePropertyName>::IsQualityLevelValid(int32 QualityLevel) const
{
	const StructType* This = StaticCast<const StructType*>(this);
	int32* Value = (int32*)This->PerQuality.Find(QualityLevel);

	if (Value != nullptr)
	{
		return true;
	}
	else
	{
		return false;
	}
}

template<typename StructType, typename ValueType, EName _BasePropertyName>
void FPerQualityLevelProperty<StructType, ValueType, _BasePropertyName>::ConvertQualityLevelDataUsingCVar(const TMap<FName, ValueType>& PlatformData, ValueType Default, bool bRequireAllPlatformsKnown)
{
	TMultiMap<FName, FName> PerPlatformToQualityLevel;

	// Make sure all platforms and groups are known before updating any of them. Missing platforms would not properly be converted to PerQuality if some of them were known and others were not.
	bool bAllPlatformsKnown = true;
	for (const TPair<FName, int32>& Pair : PlatformData)
	{
		const TArray<FName> EnginePlatformNames = QualityLevelProperty::GetEnginePlatformsForPlatformOrGroupName(Pair.Key.ToString());
		if (EnginePlatformNames.IsEmpty())
		{
			bAllPlatformsKnown = false;
			if(bRequireAllPlatformsKnown)
			{
				break;
			}
		}
		for (const FName& EnginePlatformName : EnginePlatformNames)
		{
			const FSupportedQualityLevelArray PerPlatformQualityLevels = GetSupportedQualityLevels(*EnginePlatformName.ToString());
			for (const int32 QualityLevel : PerPlatformQualityLevels)
			{
				PerPlatformToQualityLevel.Add(Pair.Key, QualityLevelProperty::QualityLevelToFName(QualityLevel));
			}
		}
	}

	if (!bRequireAllPlatformsKnown || bAllPlatformsKnown)
	{
		ConvertQualityLevelData(PlatformData, PerPlatformToQualityLevel, Default);
	}
}
#endif

/** Serializer to cook out the most appropriate platform override */
template<typename StructType, typename ValueType, EName _BasePropertyName>
void FPerQualityLevelProperty<StructType, ValueType, _BasePropertyName>::StreamArchive(FArchive& Ar)
{
	bool bCooked = false;
	StructType* This = StaticCast<StructType*>(this);
#if WITH_EDITOR
	if (Ar.IsCooking())
	{
		bCooked = true;

		FObjectSavePackageSerializeContext& ObjectSavePackageSerializeContext = Ar.GetSavePackageData()->SavePackageContext;
		EObjectSaveContextPhase CurrentSavePhase = ObjectSavePackageSerializeContext.GetPhase();
		if (CurrentSavePhase == EObjectSaveContextPhase::Harvest)
		{
			// Save data so they can be restored in the PostSave phase.
			This->SavedValue = MakePimpl<QualityLevelProperty::FSavedData<ValueType>, EPimplPtrMode::DeepCopy>();
			QualityLevelProperty::SaveQualityLevel(*This, *This->SavedValue);

			const FDataDrivenPlatformInfo& PlatformInfo = Ar.CookingTarget()->GetPlatformInfo();
			This->StripQualityLevelForCooking(*(PlatformInfo.IniPlatformName.ToString()));

			// Request post save serialization on this object so its state can be restored.
			ObjectSavePackageSerializeContext.RequestPostSaveSerialization();
		}
		else if (CurrentSavePhase == EObjectSaveContextPhase::PostSave)
		{
			if (This->SavedValue)
			{
				QualityLevelProperty::RestoreQualityLevel(*This->SavedValue, *This);
				This->SavedValue.Reset();
			}
		}
	}
#endif
	{
		Ar << bCooked;
		Ar << This->Default;
		Ar << This->PerQuality;
	}
}

/** Serializer to cook out the most appropriate platform override */
template<typename StructType, typename ValueType, EName _BasePropertyName>
void FPerQualityLevelProperty<StructType, ValueType, _BasePropertyName>::StreamStructuredArchive(FStructuredArchive::FSlot Slot)
{
	FArchive& UnderlyingArchive = Slot.GetUnderlyingArchive();
	FStructuredArchive::FRecord Record = Slot.EnterRecord();

	bool bCooked = false;
	StructType* This = StaticCast<StructType*>(this);

#if WITH_EDITOR
	if (UnderlyingArchive.IsCooking())
	{
		bCooked = true;
		This->StripQualityLevelForCooking(*(UnderlyingArchive.CookingTarget()->GetPlatformInfo().IniPlatformName.ToString()));
	}
#endif
	{
		Record << SA_VALUE(TEXT("bCooked"), bCooked);
		Record << SA_VALUE(TEXT("Value"), This->Default);
		Record << SA_VALUE(TEXT("PerQuality"), This->PerQuality);
	}
}
// 
template ENGINE_API void FPerQualityLevelProperty<FPerQualityLevelInt, int32, NAME_IntProperty>::StreamArchive(FArchive& Ar);
template ENGINE_API void FPerQualityLevelProperty<FPerQualityLevelInt, int32, NAME_IntProperty>::StreamStructuredArchive(FStructuredArchive::FSlot Slot);

#if WITH_EDITOR
template ENGINE_API int32 FPerQualityLevelProperty<FPerQualityLevelInt, int32, NAME_IntProperty>::GetValueForPlatform(const ITargetPlatform* TargetPlatform) const;
template ENGINE_API FSupportedQualityLevelArray FPerQualityLevelProperty<FPerQualityLevelInt, int32, NAME_IntProperty>::GetSupportedQualityLevels(const TCHAR* InPlatformName) const;
template ENGINE_API void FPerQualityLevelProperty<FPerQualityLevelInt, int32, NAME_IntProperty>::StripQualityLevelForCooking(const TCHAR* InPlatformName);
template ENGINE_API bool FPerQualityLevelProperty<FPerQualityLevelInt, int32, NAME_IntProperty>::IsQualityLevelValid(int32 QualityLevel) const;
template ENGINE_API void FPerQualityLevelProperty<FPerQualityLevelInt, int32, NAME_IntProperty>::ConvertQualityLevelData(const TMap<FName, int32>& PlaformData, const TMultiMap<FName, FName>& PerPlatformToQualityLevel, int32 Default);
template ENGINE_API void FPerQualityLevelProperty<FPerQualityLevelInt, int32, NAME_IntProperty>::ConvertQualityLevelDataUsingCVar(const TMap<FName, int32>& PlaformData, int32 Default, bool);
#endif
template TMap<int32, int32> QualityLevelProperty::ConvertQualityLevelData(const TMap<EPerQualityLevels, int32>& Data);
template TMap<EPerQualityLevels, int32> QualityLevelProperty::ConvertQualityLevelData(const TMap<int32, int32>& Data);
template ENGINE_API void FPerQualityLevelProperty<FPerQualityLevelInt, int32, NAME_IntProperty>::Init(const TCHAR* InCVarName, const TCHAR* InSection);
template ENGINE_API int32 FPerQualityLevelProperty<FPerQualityLevelInt, int32, NAME_IntProperty>::GetValue(int32 QualityLevel) const;

FString FPerQualityLevelInt::ToString() const
{
	FString Result = FString::FromInt(Default);

#if WITH_EDITORONLY_DATA
	TArray<int32> QualityLevels;
	PerQuality.GetKeys(QualityLevels);
	QualityLevels.Sort();

	for (int32 QL : QualityLevels)
	{
		Result = FString::Printf(TEXT("%s, %s=%d"), *Result, *QualityLevelProperty::QualityLevelToFName(QL).ToString(), PerQuality.FindChecked(QL));
	}
#endif

	return Result;
}

template ENGINE_API void FPerQualityLevelProperty<FPerQualityLevelFloat, float, NAME_FloatProperty>::StreamArchive(FArchive& Ar);
template ENGINE_API void FPerQualityLevelProperty<FPerQualityLevelFloat, float, NAME_FloatProperty>::StreamStructuredArchive(FStructuredArchive::FSlot Slot);

#if WITH_EDITOR
template int32 FPerQualityLevelProperty<FPerQualityLevelFloat, float, NAME_FloatProperty>::GetValueForPlatform(const ITargetPlatform* TargetPlatform) const;
template FSupportedQualityLevelArray FPerQualityLevelProperty<FPerQualityLevelFloat, float, NAME_FloatProperty>::GetSupportedQualityLevels(const TCHAR* InPlatformName) const;
template void FPerQualityLevelProperty<FPerQualityLevelFloat, float, NAME_FloatProperty>::StripQualityLevelForCooking(const TCHAR* InPlatformName);
template bool FPerQualityLevelProperty<FPerQualityLevelFloat, float, NAME_FloatProperty>::IsQualityLevelValid(int32 QualityLevel) const;
template ENGINE_API void FPerQualityLevelProperty<FPerQualityLevelFloat, float, NAME_FloatProperty>::ConvertQualityLevelData(const TMap<FName, float>& PlaformData, const TMultiMap<FName, FName>& PerPlatformToQualityLevel, float Default);
template ENGINE_API void FPerQualityLevelProperty<FPerQualityLevelFloat, float, NAME_FloatProperty>::ConvertQualityLevelDataUsingCVar(const TMap<FName, float>& PlaformData, float Default, bool);
#endif
template TMap<int32, float> QualityLevelProperty::ConvertQualityLevelData(const TMap<EPerQualityLevels, float>&Data);
template TMap<EPerQualityLevels, float> QualityLevelProperty::ConvertQualityLevelData(const TMap<int32, float>& Data);
template ENGINE_API void FPerQualityLevelProperty<FPerQualityLevelFloat, float, NAME_FloatProperty>::Init(const TCHAR* InCVarName, const TCHAR* InSection);
template ENGINE_API float FPerQualityLevelProperty<FPerQualityLevelFloat, float, NAME_FloatProperty>::GetValue(int32 QualityLevel) const;

FString FPerQualityLevelFloat::ToString() const
{
	FString Result = FString::Printf(TEXT("%f"), Default);

#if WITH_EDITORONLY_DATA
	TArray<int32> QualityLevels;
	PerQuality.GetKeys(QualityLevels);
	QualityLevels.Sort();

	for (int32 QL : QualityLevels)
	{
		Result = FString::Printf(TEXT("%s, %s=%f"), *Result, *QualityLevelProperty::QualityLevelToFName(QL).ToString(), PerQuality.FindChecked(QL));
	}
#endif

	return Result;
}
