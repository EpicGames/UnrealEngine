// Copyright Epic Games, Inc. All Rights Reserved.

#include "Net/PerfCountersHelpers.h"

#if USE_SERVER_PERF_COUNTERS


void ENGINE_API PerfCountersSet(const FString& Name, float Val, uint32 Flags)
{
	IPerfCounters* PerfCounters = IPerfCountersModule::Get().GetPerformanceCounters();
	if (PerfCounters)
	{
		PerfCounters->Set(Name, Val, Flags);
	}
}

void ENGINE_API PerfCountersSet(const FString& Name, double Val, uint32 Flags)
{
	IPerfCounters* PerfCounters = IPerfCountersModule::Get().GetPerformanceCounters();
	if (PerfCounters)
	{
		PerfCounters->Set(Name, Val, Flags);
	}
}

void ENGINE_API PerfCountersSet(const FString& Name, int32 Val, uint32 Flags)
{
	IPerfCounters* PerfCounters = IPerfCountersModule::Get().GetPerformanceCounters();
	if (PerfCounters)
	{
		PerfCounters->Set(Name, Val, Flags);
	}
}

void ENGINE_API PerfCountersSet(const FString& Name, const FString& Val, uint32 Flags)
{
	IPerfCounters* PerfCounters = IPerfCountersModule::Get().GetPerformanceCounters();
	if (PerfCounters)
	{
		PerfCounters->Set(Name, Val, Flags);
	}
}

int32 ENGINE_API PerfCountersIncrement(const FString & Name, int32 Add, int32 DefaultValue, uint32 Flags)
{
	IPerfCounters* PerfCounters = IPerfCountersModule::Get().GetPerformanceCounters();
	if (PerfCounters)
	{
		return PerfCounters->Increment(Name, Add, DefaultValue, Flags);
	}

	return DefaultValue + Add;
}

float ENGINE_API PerfCountersGet(const FString& Name, float DefaultVal)
{
	IPerfCounters* PerfCounters = IPerfCountersModule::Get().GetPerformanceCounters();
	if (PerfCounters)
	{
		return PerfCounters->Get(Name, DefaultVal);
	}

	return DefaultVal;
}

double ENGINE_API PerfCountersGet(const FString& Name, double DefaultVal)
{
	IPerfCounters* PerfCounters = IPerfCountersModule::Get().GetPerformanceCounters();
	if (PerfCounters)
	{
		return PerfCounters->Get(Name, DefaultVal);
	}

	return DefaultVal;
}

int32 ENGINE_API PerfCountersGet(const FString& Name, int32 DefaultVal)
{
	IPerfCounters* PerfCounters = IPerfCountersModule::Get().GetPerformanceCounters();
	if (PerfCounters)
	{
		return PerfCounters->Get(Name, DefaultVal);
	}

	return DefaultVal;
}

uint32 ENGINE_API PerfCountersGet(const FString& Name, uint32 DefaultVal)
{
	IPerfCounters* PerfCounters = IPerfCountersModule::Get().GetPerformanceCounters();
	if (PerfCounters)
	{
		return PerfCounters->Get(Name, DefaultVal);
	}

	return DefaultVal;
}

#endif // USE_SERVER_PERF_COUNTERS
