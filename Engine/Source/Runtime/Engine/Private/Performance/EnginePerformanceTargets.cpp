// Copyright Epic Games, Inc. All Rights Reserved.

#include "Performance/EnginePerformanceTargets.h"
#include "HAL/IConsoleManager.h"

//////////////////////////////////////////////////////////////////////

// The maximum threshold for an 'OK' frame time in miliseconds (*cosmetic only* and used for fps/stat display, should not be used in scalability code)
// Values above this will be red, values between this and the acceptable limit will be yellow, and values below will be green.
static TAutoConsoleVariable<float> GUnacceptableFrameTimeThresholdCVar(
	TEXT("t.UnacceptableFrameTimeThreshold"),
	50.0f,
	TEXT("The frame time theshold for what is considered completely unacceptable (in ms); values above this will be drawn as red\n")
	TEXT(" default: 50.0 ms"),
	ECVF_Scalability);

// The target threshold for frame time in miliseconds (*cosmetic only* and used for fps/stat display, should not be used in scalability code)
// Values below this will be green, values between this and the unacceptable threshold will be yellow, and values above that will be red
TAutoConsoleVariable<float> GTargetFrameTimeThresholdCVar(
	TEXT("t.TargetFrameTimeThreshold"),
	33.9f,
	TEXT("The target frame time (in ms); values below this will be drawn in green, values above will be yellow or red depending on the severity\n")
	TEXT(" default: 33.9 ms"),
	ECVF_Scalability);

// The threshold that would be considered so bad that it would cause a hitch in gameplay
// (*cosmetic only* for reporting purposes such as FPS charts, should not be used in scalability code)
FAutoConsoleVariableRef GHitchFrameTimeThresholdCVar(
	TEXT("t.HitchFrameTimeThreshold"),
	GHitchThresholdMS,
	TEXT("Definition of a hitchy frame (in ms)\n")
	TEXT(" default: 60.0 ms"),
	ECVF_Scalability);

// Minimum time passed before we'll record a new hitch
TAutoConsoleVariable<float> GHitchDeadTimeWindowCVar(
	TEXT("t.HitchDeadTimeWindow"),
	200.0f,
	TEXT("Minimum time passed before we'll record a new hitch (in ms)\n")
	TEXT(" default: 200.0 ms"),
	ECVF_Scalability);

// For the current frame to be considered a hitch, it must have run at least this many times slower than the previous frame
TAutoConsoleVariable<float> GHitchVersusNonHitchMinimumRatioCVar(
	TEXT("t.HitchVersusNonHitchRatio"),
	1.5f,
	TEXT("For the current frame to be considered a hitch, it must have run at least this many times slower than the previous frame.\nThe actual ratio is clamped to be between this and t.HitchFrameTimeThreshold/t.TargetFrameTimeThreshold\n")
	TEXT(" default: 1.5"),
	ECVF_Scalability);

//////////////////////////////////////////////////////////////////////
// FEnginePerformanceTargets

float FEnginePerformanceTargets::GetTargetFrameTimeThresholdMS()
{
	return GTargetFrameTimeThresholdCVar.GetValueOnGameThread();
}

float FEnginePerformanceTargets::GetUnacceptableFrameTimeThresholdMS()
{
	return GUnacceptableFrameTimeThresholdCVar.GetValueOnGameThread();
}

float FEnginePerformanceTargets::GetHitchFrameTimeThresholdMS()
{
	return GHitchThresholdMS;
}

float FEnginePerformanceTargets::GetMinTimeBetweenHitchesMS()
{
	return GHitchDeadTimeWindowCVar.GetValueOnGameThread();
}

float FEnginePerformanceTargets::GetHitchToNonHitchRatio()
{
	const float MinimumRatio = 1.0f;
	const float MaximumRatio = GetHitchFrameTimeThresholdMS() / GetTargetFrameTimeThresholdMS();

	return FMath::Clamp(GHitchVersusNonHitchMinimumRatioCVar.GetValueOnGameThread(), MinimumRatio, MaximumRatio);
}
