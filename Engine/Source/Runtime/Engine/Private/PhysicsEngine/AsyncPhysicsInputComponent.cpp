// Copyright Epic Games, Inc. All Rights Reserved.

#include "Physics/AsyncPhysicsInputComponent.h"

#include "Components/PrimitiveComponent.h"
#include "EngineLogs.h"
#include "EngineUtils.h"
#include "GameFramework/Pawn.h"
#include "GameFramework/PlayerController.h"
#include "PBDRigidsSolver.h"
#include "Physics/Experimental/PhysScene_Chaos.h"
#include "PhysicsProxy/SingleParticlePhysicsProxy.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AsyncPhysicsInputComponent)

namespace InputCmdCVars
{
	static int32 ForceInputDrop = 0;
	static FAutoConsoleVariableRef CVarForceInputDrop(TEXT("p.net.ForceInputDrop"), ForceInputDrop, TEXT("Forces client to drop inputs. Useful for simulating desync"));
}

DECLARE_MULTICAST_DELEGATE_ThreeParams(FOnDispatchPhysicsTick, int32 PhysicsStep, int32 NumSteps, int32 ServerFrame);

struct FAsyncPhysicsInputRewindCallback : public Chaos::IRewindCallback
{
	FAsyncPhysicsInputRewindCallback(UWorld* InWorld) : World(InWorld) { }
	UWorld* World = nullptr;

	FOnDispatchPhysicsTick DispatchPhysicsTick;

	int32 CachedServerFrame = 0;

	virtual void InjectInputs_External(int32 PhysicsStep, int32 NumSteps) override
	{
		int32 ServerFrame = PhysicsStep;
		if (World->GetNetMode() == NM_Client)
		{
			if (APlayerController* PC = World->GetFirstPlayerController())
			{
				ensure(Chaos::FPhysicsSolverBase::IsNetworkPhysicsPredictionEnabled());
				if (PC->GetNetworkPhysicsTickOffsetAssigned())
				{
					const int32 LocalToServerOffset = PC->GetNetworkPhysicsTickOffset();
					ServerFrame = PhysicsStep + LocalToServerOffset;
				}
			}
		}
		DispatchPhysicsTick.Broadcast(PhysicsStep, NumSteps, ServerFrame);
	}

	virtual int32 TriggerRewindIfNeeded_Internal(int32 LatestStepCompleted) override
	{
		int32 CachedClientFrame = INDEX_NONE;
		if (World->GetNetMode() == NM_Client && RewindData != nullptr)
		{
			CachedClientFrame = RewindData->GetResimFrame();
		}

		return CachedClientFrame;
	}

	virtual void ProcessInputs_External(int32 PhysicsStep, const TArray<Chaos::FSimCallbackInputAndObject>& SimCallbackInputs)
	{
		CachedServerFrame = PhysicsStep;

		if (World->GetNetMode() == NM_Client)
		{
			int32 LocalOffset = 0;
			if (APlayerController* PC = World->GetFirstPlayerController())
			{
				// -----------------------------------------------------------------
				// Calculate latest frame offset to map server frame to local frame
				// -----------------------------------------------------------------

				ensure(Chaos::FPhysicsSolverBase::IsNetworkPhysicsPredictionEnabled());
				if (PC->GetNetworkPhysicsTickOffsetAssigned())
				{
					const int32 LocalToServerOffset = PC->GetNetworkPhysicsTickOffset();
					CachedServerFrame = PhysicsStep + LocalToServerOffset;
				}
			}
		}
	}
};

UAsyncPhysicsInputComponent::UAsyncPhysicsInputComponent()
{
	bAutoActivate = true;
	SetIsReplicatedByDefault(true);
}

void UAsyncPhysicsInputComponent::SetDataClass(TSubclassOf<UAsyncPhysicsData> InDataClass)
{
	ensureMsgf(DataClass == nullptr, TEXT("You can only set the data class once"));
	DataClass = InDataClass;

	DataToWrite = DuplicateObject((UAsyncPhysicsData*)DataClass->GetDefaultObject(), nullptr);
	
	//now that we have a class we're ready to async tick
	SetAsyncPhysicsTickEnabled(true);
	UWorld* World = GetWorld();
	if (World->WorldType == EWorldType::PIE || World->WorldType == EWorldType::Game)
	{
		FPhysScene* PhysScene = World->GetPhysicsScene();
		if (ensureAlways(PhysScene))
		{
			Chaos::FPhysicsSolver* Solver = PhysScene->GetSolver();
			if (Solver->GetRewindCallback() == nullptr)
			{
				// -------------------------------------------------------------------------------------
				// Hack: enable Rewind capture here. This needs to be moved somewhere else
				// -------------------------------------------------------------------------------------

				const int32 PhysicsHistoryLength = FChaosSolversModule::GetModule()->GetSettingsProvider().GetPhysicsHistoryCount();

				Solver->EnableRewindCapture(PhysicsHistoryLength, true, MakeUnique<FAsyncPhysicsInputRewindCallback>(World));
			}

			FAsyncPhysicsInputRewindCallback* Callback = static_cast<FAsyncPhysicsInputRewindCallback*>(Solver->GetRewindCallback());
			Callback->DispatchPhysicsTick.AddUObject(this, &UAsyncPhysicsInputComponent::OnDispatchPhysicsTick);
		}
	}
}

void UAsyncPhysicsInputComponent::AsyncPhysicsTickComponent(float DeltaTime, float SimTime)
{
	QUICK_SCOPE_CYCLE_COUNTER(AsyncPhysicsInputComponent_AsyncPhysicsTick);

	Super::AsyncPhysicsTickComponent(DeltaTime, SimTime);
	ensureMsgf(DataClass != nullptr, TEXT("You must call SetDataClass after creating the component"));

	UWorld* World = GetWorld();
	FPhysScene* PhysScene = World->GetPhysicsScene();
	Chaos::FPhysicsSolver* Solver = PhysScene->GetSolver();

	// Pull server frame out in this unsafe way. Better if we can get piped into AsyncPhysicsTickComponent
	FAsyncPhysicsInputRewindCallback* Callback = static_cast<FAsyncPhysicsInputRewindCallback*>(Solver->GetRewindCallback());
	const int32 ServerFrame = Callback->CachedServerFrame;

	DataToConsume = nullptr; //set current input to none in case we don't find it

	for (int32 Idx = BufferedData.Num() - 1; Idx >= 0; --Idx)
	{
		UAsyncPhysicsData* Data = BufferedData[Idx];
		if (Data->ServerFrame == ServerFrame)
		{
			DataToConsume = Data;
		}

		bool bFreeData = Data->ServerFrame < ServerFrame;	//for most cases once we're passed this frame we can free
		if (APlayerController* PC = GetPlayerController())
		{
			if (PC->IsLocalController() && !InputCmdCVars::ForceInputDrop)
			{
				//TODO: fix rpc version
				//ServerRPCBufferInput(Data);

				//If we are the local player then we need to keep this data around to send redundant RPCs to deal with potential packet loss
				bFreeData = --Data->ReplicationRedundancy <= 0;
			}
		}

		if (bFreeData)
		{
			BufferedData.RemoveAtSwap(Idx);
		}
	}
}

void UAsyncPhysicsInputComponent::ServerRPCBufferInput_Implementation(UAsyncPhysicsData* NewData)
{
	for (UAsyncPhysicsData* Data : BufferedData)
	{
		if (Data->ServerFrame == NewData->ServerFrame)
		{
			//already buffered this is just a redundant send to deal with potential packet loss
			//question: what if client's offset changes so we get a new instruction for what we think is an existing frame? The case where offset changes is input fault (hopefully rare) so maybe we just accept it
			return;
		}
	}

	//TODO: fix rpc version
	//BufferedData.Add(NewData);
}

void UAsyncPhysicsInputComponent::OnDispatchPhysicsTick(int32 PhysicsStep, int32 NumSteps, int32 ServerFrame)
{
	ensureMsgf(DataClass != nullptr, TEXT("You must call SetDataClass after creating the component"));
	if (!ensure(DataToWrite != nullptr))
	{
		return;
	}
	
	// It would be better if we only registered while we had a local controller,
	// but this is simpler to implement

	if (APlayerController* PC = GetPlayerController())
	{
		if (PC->IsLocalController())
		{
			UAsyncPhysicsData* DataToSend = DataToWrite;
			DataToWrite = DuplicateObject((UAsyncPhysicsData*)DataClass->GetDefaultObject(), nullptr);
			for (int32 Step = 0; Step < NumSteps; ++Step)
			{
				if(Step > 0)
				{
					//Make sure each sub-step gets its own unique data so we don't have to worry about ref count
					//TODO: should probably given user opportunity to modify this per sub-step. For example a jump instruction should only happen on first sub-step
					DataToSend = DuplicateObject(DataToSend, nullptr);
				}
				DataToSend->ServerFrame = ServerFrame + 1 + Step;
				BufferedData.Add(DataToSend);
			}
		}
	}
}

APlayerController* UAsyncPhysicsInputComponent::GetPlayerController()
{
	if (APlayerController* PC = Cast<APlayerController>(GetOwner()))
	{
		return PC;
	}

	if (APawn* Pawn = Cast<APawn>(GetOwner()))
	{
		return Pawn->GetController<APlayerController>();
	}

	return nullptr;
}


UAsyncPhysicsData* UAsyncPhysicsInputComponent::GetDataToWrite() const
{
	//TODO: ensure not inside async physics tick
	return DataToWrite;
}

const UAsyncPhysicsData* UAsyncPhysicsInputComponent::GetDataToConsume() const
{
	//TODO: ensure inside async physics tick
	ensureMsgf(DataClass != nullptr, TEXT("You must call SetDataClass after creating the component"));
	if (DataToConsume)
	{
		return DataToConsume;
	}
	
	return (UAsyncPhysicsData*)DataClass->GetDefaultObject();
}

