// Copyright Epic Games, Inc. All Rights Reserved.

#include "PhysicsEngine/BodyInstance.h"
#include "BodySetupEnums.h"
#include "Components/StaticMeshComponent.h"
#include "Chaos/CollisionConvexMesh.h"
#include "Engine/Engine.h"
#include "Components/SkeletalMeshComponent.h"
#include "Components/ShapeComponent.h"
#include "GameFramework/Actor.h"
#include "Engine/CollisionProfile.h"
#include "Engine/OverlapResult.h"
#include "Engine/World.h"
#include "PrimitiveDrawingUtils.h"
#include "Collision.h"
#include "Materials/MaterialInterface.h"
#include "Physics/PhysicsFiltering.h"
#include "Physics/Experimental/PhysScene_Chaos.h"
#include "PhysicsEngine/ConstraintInstance.h"
#include "Physics/PhysicsInterfaceTypes.h"
#include "PhysicsEngine/ConvexElem.h"
#include "PhysicsEngine/BoxElem.h"
#include "PhysicsEngine/LevelSetElem.h"
#include "PhysicsEngine/SphereElem.h"
#include "PhysicsEngine/SphylElem.h"
#include "PhysicsEngine/BodySetup.h"
#include "PhysicsEngine/BodyUtils.h"
#include "PhysicsEngine/PhysicsBodyInstanceOwnerInterface.h"
#include "Logging/MessageLog.h"

#include "Chaos/Capsule.h"
#include "Chaos/Convex.h"
#include "Chaos/TriangleMeshImplicitObject.h"
#include "Chaos/AsyncInitBodyHelper.h"
#include "Physics/Experimental/ChaosInterfaceUtils.h"

#include "PhysicsEngine/TaperedCapsuleElem.h"
#include "PhysicsProxy/SingleParticlePhysicsProxy.h"

#include "Components/BrushComponent.h"
#include "PhysicalMaterials/PhysicalMaterial.h"
#include "PhysicalMaterials/PhysicalMaterialMask.h"
#include "PhysicsEngine/PhysicsSettings.h"

#define LOCTEXT_NAMESPACE "BodyInstance"

#include "Chaos/ChaosEngineInterface.h"
#include "Physics/Experimental/ChaosScopedSceneLock.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(BodyInstance)

DECLARE_CYCLE_STAT(TEXT("Init Body"), STAT_InitBody, STATGROUP_Physics);
DECLARE_CYCLE_STAT(TEXT("Init Body Debug"), STAT_InitBodyDebug, STATGROUP_Physics);
DECLARE_CYCLE_STAT(TEXT("Init Body Scene Interaction"), STAT_InitBodySceneInteraction, STATGROUP_Physics);
DECLARE_CYCLE_STAT(TEXT("Init Body Aggregate"), STAT_InitBodyAggregate, STATGROUP_Physics);
DECLARE_CYCLE_STAT(TEXT("Init Body Add"), STAT_InitBodyAdd, STATGROUP_Physics);
DECLARE_CYCLE_STAT(TEXT("Init Body Post Add to Scene"), STAT_InitBodyPostAdd, STATGROUP_Physics);
DECLARE_CYCLE_STAT(TEXT("Term Body"), STAT_TermBody, STATGROUP_Physics);
DECLARE_CYCLE_STAT(TEXT("Update Materials"), STAT_UpdatePhysMats, STATGROUP_Physics);
DECLARE_CYCLE_STAT(TEXT("Update Materials Scene Interaction"), STAT_UpdatePhysMatsSceneInteraction, STATGROUP_Physics);
DECLARE_CYCLE_STAT_WITH_FLAGS(TEXT("Filter Update"), STAT_UpdatePhysFilter, STATGROUP_Physics, EStatFlags::Verbose);
DECLARE_CYCLE_STAT(TEXT("Filter Update (PhysX Code)"), STAT_UpdatePhysFilterPhysX, STATGROUP_Physics);
DECLARE_CYCLE_STAT(TEXT("Init Bodies"), STAT_InitBodies, STATGROUP_Physics);
DECLARE_CYCLE_STAT(TEXT("Bulk Body Scene Add"), STAT_BulkSceneAdd, STATGROUP_Physics);
DECLARE_CYCLE_STAT(TEXT("Static Init Bodies"), STAT_StaticInitBodies, STATGROUP_Physics);
DECLARE_CYCLE_STAT(TEXT("UpdateBodyScale"), STAT_BodyInstanceUpdateBodyScale, STATGROUP_Physics);
DECLARE_CYCLE_STAT(TEXT("CreatePhysicsShapesAndActors"), STAT_CreatePhysicsShapesAndActors, STATGROUP_Physics);
DECLARE_CYCLE_STAT(TEXT("CreatePhysicsShapes"), STAT_CreatePhysicsShapes, STATGROUP_Physics);
DECLARE_CYCLE_STAT(TEXT("CreatePhysicsActor"), STAT_CreatePhysicsActor, STATGROUP_Physics);
DECLARE_CYCLE_STAT_WITH_FLAGS(TEXT("BodyInstance SetCollisionProfileName"), STAT_BodyInst_SetCollisionProfileName, STATGROUP_Physics, EStatFlags::Verbose);
DECLARE_CYCLE_STAT(TEXT("Phys SetBodyTransform"), STAT_SetBodyTransform, STATGROUP_Physics);

// @HACK Guard to better encapsulate game related hacks introduced into UpdatePhysicsFilterData()
TAutoConsoleVariable<int32> CVarEnableDynamicPerBodyFilterHacks(
	TEXT("p.EnableDynamicPerBodyFilterHacks"), 
	0, 
	TEXT("Enables/Disables the use of a set of game focused hacks - allowing users to modify skel body collision dynamically (changes the behavior of per-body collision filtering)."),
	ECVF_ReadOnly
);

TAutoConsoleVariable<int32> CVarIgnoreAnalyticCollisionsOverride(
	TEXT("p.IgnoreAnalyticCollisionsOverride"), 
	0, 
	TEXT("Overrides the default for ignroing analytic collsions."),
	ECVF_ReadOnly
);

bool bPreventInvalidBodyInstanceTransforms = true;
FAutoConsoleVariableRef CVarbPreventInvalidBodyInstanceTransforms(
	TEXT("p.PreventInvalidBodyInstanceTransforms"), 
	bPreventInvalidBodyInstanceTransforms, 
	TEXT("If true, an attempt to create a BodyInstance with an invalid transform will fail with a warning"));

bool bEnableOverrideSolverDeltaTime = true;
FAutoConsoleVariableRef CVarbEnableOverrideSolverDeltaTime(
	TEXT("p.EnableOverrideSolverDeltaTime"),
	bEnableOverrideSolverDeltaTime,
	TEXT("If true, setting for override solver delta time can be used.  False will disable this feature."));

bool bSkipShapeCreationForEmptyBodySetup = false;
FAutoConsoleVariableRef CVarSkipShapeCreationForEmptyBodySetup(
	TEXT("p.SkipShapeCreationForEmptyBodySetup"),
	bSkipShapeCreationForEmptyBodySetup,
	TEXT("If true, CreateShapesAndActors will not try to create actors and shapes for all instances if the body setup doesn't have any geometry."));

float SensitiveSleepThresholdMultiplier = 1.0f/20.0f;
FAutoConsoleVariableRef CVarSensitiveSleepThresholdMultiplier(
	TEXT("p.SensitiveSleepThresholdMultiplier"),
	SensitiveSleepThresholdMultiplier,
	TEXT("The sleep threshold multiplier to use for bodies using the Sensitive sleep family."));

bool bClearPerShapeOverridesOnTerminate = true;
FAutoConsoleVariableRef CVarClearPerShapeOverridesOnTerminate(
	TEXT("p.ClearPerShapeOverridesOnTerminate"),
	bClearPerShapeOverridesOnTerminate,
	TEXT("Whether shape overrides are cleared in between recreations of a body instance"));

using namespace PhysicsInterfaceTypes;

bool IsRigidBodyKinematic_AssumesLocked(const FPhysicsActorHandle& InActorRef)
{
	if(FPhysicsInterface::IsRigidBody(InActorRef))
	{
		return FPhysicsInterface::IsKinematic_AssumesLocked(InActorRef);
	}

	return false;
}

int32 FillInlineShapeArray_AssumesLocked(PhysicsInterfaceTypes::FInlineShapeArray& Array, const FPhysicsActorHandle& Actor)
{
	FPhysicsInterface::GetAllShapes_AssumedLocked(Actor, Array);

	return Array.Num();
}

////////////////////////////////////////////////////////////////////////////
// FCollisionResponse
////////////////////////////////////////////////////////////////////////////

FCollisionResponse::FCollisionResponse()
{

}

FCollisionResponse::FCollisionResponse(ECollisionResponse DefaultResponse)
{
	SetAllChannels(DefaultResponse);
}

/** Set the response of a particular channel in the structure. */
bool FCollisionResponse::SetResponse(ECollisionChannel Channel, ECollisionResponse NewResponse)
{
#if 1// @hack until PostLoad is disabled for CDO of BP WITH_EDITOR
	ECollisionResponse DefaultResponse = FCollisionResponseContainer::GetDefaultResponseContainer().GetResponse(Channel);
	if (DefaultResponse == NewResponse)
	{
		RemoveReponseFromArray(Channel);
	}
	else
	{
		AddReponseToArray(Channel, NewResponse);
	}
#endif

	return ResponseToChannels.SetResponse(Channel, NewResponse);
}

/** Set all channels to the specified response */
bool FCollisionResponse::SetAllChannels(ECollisionResponse NewResponse)
{
	if (ResponseToChannels.SetAllChannels(NewResponse))
	{
#if 1// @hack until PostLoad is disabled for CDO of BP WITH_EDITOR
		UpdateArrayFromResponseContainer();
#endif
		return true;
	}
	return false;
}

bool FCollisionResponse::ReplaceChannels(ECollisionResponse OldResponse, ECollisionResponse NewResponse)
{
	if (ResponseToChannels.ReplaceChannels(OldResponse, NewResponse))
	{
#if 1// @hack until PostLoad is disabled for CDO of BP WITH_EDITOR
		UpdateArrayFromResponseContainer();
#endif
		return true;
	}
	return false;
}

/** Set all channels from ChannelResponse Array **/
bool FCollisionResponse::SetCollisionResponseContainer(const FCollisionResponseContainer& InResponseToChannels)
{
	if (ResponseToChannels != InResponseToChannels)
	{
		ResponseToChannels = InResponseToChannels;
#if 1// @hack until PostLoad is disabled for CDO of BP WITH_EDITOR
		// this is only valid case that has to be updated
		UpdateArrayFromResponseContainer();
#endif
		return true;
	}
	return false;
}

void FCollisionResponse::SetResponsesArray(const TArray<FResponseChannel>& InChannelResponses)
{
#if DO_GUARD_SLOW
	// verify if the name is overlapping, if so, ensure, do not remove in debug because it will cause inconsistent bug between debug/release
	int32 const ResponseNum = InChannelResponses.Num();
	for (int32 I=0; I<ResponseNum; ++I)
	{
		for (int32 J=I+1; J<ResponseNum; ++J)
		{
			if (InChannelResponses[I].Channel == InChannelResponses[J].Channel)
			{
				UE_LOGF(LogCollision, Warning, "Collision Channel : Redundant name exists");
			}
		}
	}
#endif

	ResponseArray = InChannelResponses;
	UpdateResponseContainerFromArray();
}

#if 1// @hack until PostLoad is disabled for CDO of BP WITH_EDITOR
bool FCollisionResponse::RemoveReponseFromArray(ECollisionChannel Channel)
{
	// this is expensive operation, I'd love to remove names but this operation is supposed to do
	// so only allow it in editor
	// without editor, this does not have to match 
	// We'd need to save name just in case that name is gone or not
	FName ChannelName = UCollisionProfile::Get()->ReturnChannelNameFromContainerIndex(Channel);
	for (auto Iter=ResponseArray.CreateIterator(); Iter; ++Iter)
	{
		if (ChannelName == (*Iter).Channel)
		{
			ResponseArray.RemoveAt(Iter.GetIndex());
			return true;
		}
	}
	return false;
}

bool FCollisionResponse::AddReponseToArray(ECollisionChannel Channel, ECollisionResponse Response)
{
	// this is expensive operation, I'd love to remove names but this operation is supposed to do
	// so only allow it in editor
	// without editor, this does not have to match 
	FName ChannelName = UCollisionProfile::Get()->ReturnChannelNameFromContainerIndex(Channel);
	for (auto Iter=ResponseArray.CreateIterator(); Iter; ++Iter)
	{
		if (ChannelName == (*Iter).Channel)
		{
			(*Iter).Response = Response;
			return true;
		}
	}

	// if not add to the list
	ResponseArray.Add(FResponseChannel(ChannelName, Response));
	return true;
}

void FCollisionResponse::UpdateArrayFromResponseContainer()
{
	ResponseArray.Empty(UE_ARRAY_COUNT(ResponseToChannels.EnumArray));

	const FCollisionResponseContainer& DefaultResponse = FCollisionResponseContainer::GetDefaultResponseContainer();
	const UCollisionProfile* CollisionProfile = UCollisionProfile::Get();

	for (int32 i = 0; i < UE_ARRAY_COUNT(ResponseToChannels.EnumArray); i++)
	{
		// if not same as default
		if (ResponseToChannels.EnumArray[i] != DefaultResponse.EnumArray[i])
		{
			FName ChannelName = CollisionProfile->ReturnChannelNameFromContainerIndex(i);
			if (ChannelName != NAME_None)
			{
				FResponseChannel NewResponse;
				NewResponse.Channel = ChannelName;
				NewResponse.Response = (ECollisionResponse)ResponseToChannels.EnumArray[i];
				ResponseArray.Add(NewResponse);
			}
		}
	}

	// Shrink array as its slack memory might not be used and might be wasted over time.
	ResponseArray.Shrink();
}

#endif // WITH_EDITOR

void FCollisionResponse::UpdateResponseContainerFromArray()
{
	ResponseToChannels = FCollisionResponseContainer::GetDefaultResponseContainer();

	for (auto Iter = ResponseArray.CreateIterator(); Iter; ++Iter)
	{
		FResponseChannel& Response = *Iter;

		int32 EnumIndex = UCollisionProfile::Get()->ReturnContainerIndexFromChannelName(Response.Channel);
		if ( EnumIndex != INDEX_NONE )
		{
			ResponseToChannels.SetResponse((ECollisionChannel)EnumIndex, Response.Response);
		}
		else
		{
			// otherwise remove
			ResponseArray.RemoveAt(Iter.GetIndex());
			--Iter;
		}
	}
}

bool FCollisionResponse::operator==(const FCollisionResponse& Other) const
{
	bool bCollisionResponseEqual = ResponseArray.Num() == Other.ResponseArray.Num();
	if(bCollisionResponseEqual)
	{
		for(int32 ResponseIdx = 0; ResponseIdx < ResponseArray.Num(); ++ResponseIdx)
		{
			bool bFound = false;
			for(int32 InternalIdx = 0; InternalIdx < ResponseArray.Num(); ++InternalIdx)
			{
				if(ResponseArray[ResponseIdx].Channel == Other.ResponseArray[InternalIdx].Channel)
				{
					if (ResponseArray[ResponseIdx].Response != Other.ResponseArray[InternalIdx].Response)
					{
						return false;
					}
					bFound = true;
					break;
				}
			}
			if(!bFound)
			{
				return false;
			}
		}
	}

	return bCollisionResponseEqual;
}

FBodyInstance::FAsyncTermBodyPayload::FAsyncTermBodyPayload()
: ActorHandle(DefaultPhysicsActorHandle())
{
}
////////////////////////////////////////////////////////////////////////////


FBodyInstance::FBodyInstance()
	: InstanceBodyIndex(INDEX_NONE)
	, InstanceBoneIndex(INDEX_NONE)
	, PositionSolverIterationCount(8)
	, VelocitySolverIterationCount(2)
	, ProjectionSolverIterationCount(1)
	, ObjectType(ECC_WorldStatic)
	, MaskFilter(0)
	, CollisionEnabled(ECollisionEnabled::QueryAndPhysics)
	, CurrentSceneState(BodyInstanceSceneState::NotAdded)
	, SleepFamily(ESleepFamily::Normal)
	, DOFMode(0)
	, bUseCCD(false)
	, bShapeCollisionEnabledIsSet(false)
	, bShapeCollisionResponsesIsSet(false)
	, bUseMACD(false)
	, bAllowPartialIslandSleep(true)
	, bIgnoreAnalyticCollisions(false)
	, bNotifyRigidBodyCollision(false)
	, bContactModification(false)
	, bSmoothEdgeCollisions(false)
	, bLockTranslation(true)
	, bLockRotation(true)
	, bLockXTranslation(false)
	, bLockYTranslation(false)
	, bLockZTranslation(false)
	, bLockXRotation(false)
	, bLockYRotation(false)
	, bLockZRotation(false)
	, bOverrideMaxAngularVelocity(false)
	, bOverrideMaxDepenetrationVelocity(false)
	, bOverrideWalkableSlopeOnInstance(false)
	, bOverrideIterationCounts(false)
	, bInterpolateWhenSubStepping(true)
	, bPendingCollisionProfileSetup(false)
	, bInertiaConditioning(true)	
	, bOneWayInteraction(false)
	, bOverrideSolverAsyncDeltaTime(false)
	, SolverAsyncDeltaTime(1.f / 60)
	, CollisionProfileName(UCollisionProfile::CustomCollisionProfileName)
	, Scale3D(1.0f)
	, MaxDepenetrationVelocity(0.f)
	, MassInKgOverride(100.f)
	, ExternalCollisionProfileBodySetup(nullptr)
	, LinearDamping(0.01f)
	, AngularDamping(0.0)
	, CustomDOFPlaneNormal(FVector::ZeroVector)
	, COMNudge(ForceInit)
	, MassScale(1.f)
	, GravityGroupIndex(0)
	, InertiaTensorScale(1.f)
	, DOFConstraint(nullptr)
	, WeldParent(nullptr)
	, PhysMaterialOverride(nullptr)
	, CustomSleepThresholdMultiplier(1.f)
	, StabilizationThresholdMultiplier(1.f)
	, PhysicsBlendWeight(0.f)
	, ActorHandle(DefaultPhysicsActorHandle())
	, BodyInstanceOwner(nullptr)
{
	MaxAngularVelocity = UPhysicsSettings::Get()->MaxAngularVelocity;
}

FBodyInstance::~FBodyInstance() = default;

FArchive& operator<<(FArchive& Ar,FBodyInstance& BodyInst)
{
	if (!Ar.IsLoading() && !Ar.IsSaving())
	{
		Ar << BodyInst.OwnerComponent;
		Ar << BodyInst.PhysMaterialOverride;
	}

	if (Ar.IsLoading() && Ar.UEVer() < VER_UE4_MAX_ANGULAR_VELOCITY_DEFAULT)
	{
		if(BodyInst.MaxAngularVelocity != 400.f)
		{
			BodyInst.bOverrideMaxAngularVelocity = true;
		}
	}

	return Ar;
}


/** Util for finding the parent bodyinstance of a specified body, using skeleton hierarchy */
FBodyInstance* FindParentBodyInstance(FName BodyName, USkeletalMeshComponent* SkelMeshComp)
{
	FName TestBoneName = BodyName;
	while(true)
	{
		TestBoneName = SkelMeshComp->GetParentBone(TestBoneName);
		// Bail out if parent bone not found
		if(TestBoneName == NAME_None)
		{
			return NULL;
		}

		// See if we have a body for the parent bone
		FBodyInstance* BI = SkelMeshComp->GetBodyInstance(TestBoneName);
		if(BI != NULL)
		{
			// We do - return it
			return BI;
		}

		// Don't repeat if we are already at the root!
		if(SkelMeshComp->GetBoneIndex(TestBoneName) == 0)
		{
			return NULL;
		}
	}

	return NULL;
}

//Determine that the shape is associated with this subbody (or root body)
bool FBodyInstance::IsShapeBoundToBody(const FPhysicsShapeHandle& Shape) const
{
	const FBodyInstance* BI = GetOriginalBodyInstance(Shape);
	return BI == this;
}

const TMap<FPhysicsShapeHandle, FBodyInstance::FWeldInfo>* FBodyInstance::GetCurrentWeldInfo() const
{
	return ShapeToBodiesMap.Get();
}

int32 FBodyInstance::GetAllShapes_AssumesLocked(TArray<FPhysicsShapeHandle>& OutShapes) const
{
	if(GetPhysicsActor())
	{
		return FPhysicsInterface::GetAllShapes_AssumedLocked(GetPhysicsActor(), OutShapes);
	}

	return 0;
}

void FBodyInstance::UpdateTriMeshVertices(const TArray<FVector> & NewPositions)
{
	if (BodySetup.IsValid())
	{
		FPhysicsCommand::ExecuteWrite(GetPhysicsActor(), [&](FPhysicsActorHandle Actor)
		{
			//after updating the vertices we must call setGeometry again to update any shapes referencing the mesh
			TArray<FPhysicsShapeHandle> Shapes;
			const int32 SyncShapeCount = GetAllShapes_AssumesLocked(Shapes);

			for (FPhysicsShapeHandle& Shape : Shapes)
			{
				if (FPhysicsInterface::GetShapeType(Shape) == ECollisionShapeType::Trimesh)
				{
					using namespace Chaos;
					const Chaos::FImplicitObject* ShapeImplicit = Shape.Shape->GetGeometry();
					EImplicitObjectType Type = ShapeImplicit->GetType();

					// Cast to derived implicit, copy trianglemesh.
					FVec3 Scale(1, 1, 1);
					FImplicitObjectPtr TriMeshCopy = nullptr;
					if (IsInstanced(Type))
					{
						const TImplicitObjectInstanced<FTriangleMeshImplicitObject>& InstancedImplicit = ShapeImplicit->GetObjectChecked<TImplicitObjectInstanced<FTriangleMeshImplicitObject>>();
						const FTriangleMeshImplicitObject* TriangleMesh = InstancedImplicit.GetInstancedObject();
						TriMeshCopy = TriangleMesh->DeepCopyGeometry();
					}
					else if (IsScaled(Type))
					{
						const TImplicitObjectScaled<FTriangleMeshImplicitObject>& ScaledImplicit = ShapeImplicit->GetObjectChecked<TImplicitObjectScaled<FTriangleMeshImplicitObject>>();
						const FTriangleMeshImplicitObject* TriangleMesh = ScaledImplicit.GetUnscaledObject();
						Scale = ScaledImplicit.GetScale();
						TriMeshCopy = TriangleMesh->DeepCopyGeometry();
					}
					else
					{
						const FTriangleMeshImplicitObject& TriangleMesh = ShapeImplicit->GetObjectChecked<FTriangleMeshImplicitObject>();
						TriMeshCopy = TriangleMesh.DeepCopyGeometry();
					}
					FTriangleMeshImplicitObjectPtr TriMeshCopyPtr(TriMeshCopy->GetObject<FTriangleMeshImplicitObject>());
					TriMeshCopyPtr->UpdateVertices(NewPositions);
					if (Scale != FVec3(1, 1, 1))
					{
						Chaos::FImplicitObjectPtr Scaled = MakeImplicitObjectPtr<TImplicitObjectScaled<FTriangleMeshImplicitObject, /*bInstanced=*/false>>(MoveTemp(TriMeshCopyPtr), Scale);
						FPhysicsInterface::SetGeometry(Shape, MoveTemp(Scaled));
					}
					else
					{
						FPhysicsInterface::SetGeometry(Shape, MoveTemp(TriMeshCopy));
					}
				}
			}
		});
	}
}

void FBodyInstance::UpdatePhysicalMaterials()
{
	SCOPE_CYCLE_COUNTER(STAT_UpdatePhysMats);
	UPhysicalMaterial* SimplePhysMat = GetSimplePhysicalMaterial();
	TArray<FPhysicalMaterialMaskParams> ComplexPhysMatMasks;
	TArray<UPhysicalMaterial*> ComplexPhysMats = GetComplexPhysicalMaterials(ComplexPhysMatMasks);

	FPhysicsCommand::ExecuteWrite(GetPhysicsActorWeldRoot(), [&](const FPhysicsActorHandle& Actor)
	{
		ApplyMaterialToInstanceShapes_AssumesLocked(SimplePhysMat, ComplexPhysMats, ComplexPhysMatMasks);
	});
}

void FBodyInstance::InvalidateCollisionProfileName()
{
	CollisionProfileName = UCollisionProfile::CustomCollisionProfileName;
	ExternalCollisionProfileBodySetup = nullptr;
	bPendingCollisionProfileSetup = false;
}

bool FBodyInstance::SetResponseToChannel(ECollisionChannel Channel, ECollisionResponse NewResponse)
{
	if (CollisionResponses.SetResponse(Channel, NewResponse))
	{
		InvalidateCollisionProfileName();
		UpdatePhysicsFilterData();
		return true;
	}

	return false;
}

bool FBodyInstance::SetResponseToAllChannels(ECollisionResponse NewResponse)
{
	if (CollisionResponses.SetAllChannels(NewResponse))
	{
		InvalidateCollisionProfileName();
		UpdatePhysicsFilterData();
		return true;
	}

	return false;
}
	
bool FBodyInstance::ReplaceResponseToChannels(ECollisionResponse OldResponse, ECollisionResponse NewResponse)
{
	if (CollisionResponses.ReplaceChannels(OldResponse, NewResponse))
	{
		InvalidateCollisionProfileName();
		UpdatePhysicsFilterData();
		return true;
	}

	return false;
}

bool FBodyInstance::SetResponseToChannels(const FCollisionResponseContainer& NewResponses)
{
	if (CollisionResponses.SetCollisionResponseContainer(NewResponses))
	{
		InvalidateCollisionProfileName();
		UpdatePhysicsFilterData();
		return true;
	}

	return false;
}

bool FBodyInstance::SetShapeResponseToChannels(const int32 ShapeIndex, const FCollisionResponseContainer& NewResponses)
{
	if (!bShapeCollisionResponsesIsSet)
	{
		ShapeCollisionResponses.Reset();
		bShapeCollisionResponsesIsSet = true;
	}

	bool bIndexExists = false;
	int32 ResponseIndex = 0;
	const int32 ResponseNum = ShapeCollisionResponses.Num();
	for (; ResponseIndex < ResponseNum; ++ResponseIndex)
	{
		if (ShapeCollisionResponses[ResponseIndex].Key == ShapeIndex)
		{
			break;
		}
	}

	if (ResponseIndex == ResponseNum)
	{
		ShapeCollisionResponses.Add(TPair<int32, FCollisionResponse>(ShapeIndex, FCollisionResponse()));
	}

	if (ShapeCollisionResponses[ResponseIndex].Value.SetCollisionResponseContainer(NewResponses))
	{
		UpdatePhysicsFilterData();
		return true;
	}

	return false;
}

const FCollisionResponseContainer& FBodyInstance::GetShapeResponseToChannels(const int32 ShapeIndex) const
{
	return GetShapeResponseToChannels(ShapeIndex, GetResponseToChannels());
}

const FCollisionResponseContainer& FBodyInstance::GetShapeResponseToChannels(const int32 ShapeIndex, const FCollisionResponseContainer& DefaultResponseContainer) const
{
	// Return per-shape collision response override if there is one
	if (bShapeCollisionResponsesIsSet)
	{
		for (int32 ResponseIndex = 0; ResponseIndex < ShapeCollisionResponses.Num(); ++ResponseIndex)
		{
			if (ShapeCollisionResponses[ResponseIndex].Key == ShapeIndex)
			{
				return ShapeCollisionResponses[ResponseIndex].Value.GetResponseContainer();
			}
		}
	}

	// Return base collision response
	return DefaultResponseContainer;
}

void FBodyInstance::SetObjectType(ECollisionChannel Channel)
{
	InvalidateCollisionProfileName();
	ObjectType = Channel;
	UpdatePhysicsFilterData();
}

void FBodyInstance::ApplyDeferredCollisionProfileName()
{
	if(bPendingCollisionProfileSetup)
	{
		SetCollisionProfileName(CollisionProfileName);
		bPendingCollisionProfileSetup = false;
	}
}

void FBodyInstance::SetCollisionProfileNameDeferred(FName InCollisionProfileName)
{
	CollisionProfileName = InCollisionProfileName;
	ExternalCollisionProfileBodySetup = nullptr;
	bPendingCollisionProfileSetup = true;
}

void FBodyInstance::SetCollisionProfileName(FName InCollisionProfileName)
{
	SCOPE_CYCLE_COUNTER(STAT_BodyInst_SetCollisionProfileName);

	//Note that GetCollisionProfileName will use the external profile if one is set.
	//GetCollisionProfileName will be consistent with the values set by LoadProfileData.
	//This is why we can't use CollisionProfileName directly during the equality check
	if (bPendingCollisionProfileSetup || GetCollisionProfileName() != InCollisionProfileName)
	{
		//LoadProfileData uses GetCollisionProfileName internally so we must now set the external collision data to null.
		ExternalCollisionProfileBodySetup = nullptr;
		CollisionProfileName = InCollisionProfileName;
		// now Load ProfileData
		LoadProfileData(false);

		bPendingCollisionProfileSetup = false;
	}
	
	ExternalCollisionProfileBodySetup = nullptr;	//Even if incoming is the same as GetCollisionProfileName we turn it into "manual mode"
}

FName FBodyInstance::GetCollisionProfileName() const
{
	FName ReturnProfileName = CollisionProfileName;
	if (UBodySetup* BodySetupPtr = ExternalCollisionProfileBodySetup.Get(true))
	{
		ReturnProfileName = BodySetupPtr->DefaultInstance.CollisionProfileName;
	}
	
	return ReturnProfileName;
}


bool FBodyInstance::DoesUseCollisionProfile() const
{
	return IsValidCollisionProfileName(GetCollisionProfileName());
}

void FBodyInstance::SetMassScale(float InMassScale)
{
	MassScale = InMassScale;
	UpdateMassProperties();
}

void FBodyInstance::SetCollisionEnabled(ECollisionEnabled::Type NewType, bool bUpdatePhysicsFilterData)
{
	if (CollisionEnabled != NewType)
	{
		ECollisionEnabled::Type OldType = CollisionEnabled;
		InvalidateCollisionProfileName();
		CollisionEnabled = NewType;
		
		// Only update physics filter data if required
		if (bUpdatePhysicsFilterData)
		{
			UpdatePhysicsFilterData();
		}

		bool bWasPhysicsEnabled = CollisionEnabledHasPhysics(OldType);
		bool bIsPhysicsEnabled = CollisionEnabledHasPhysics(NewType);

		// Whenever we change physics state, call Recreate
		// This should also handle destroying the state (in case it's newly disabled).
		if (bWasPhysicsEnabled != bIsPhysicsEnabled)
		{
			if(UPrimitiveComponent* PrimComponent = OwnerComponent.Get())
			{
				PrimComponent->RecreateInstanceBody(InstanceBodyIndex);
			}

		}

	}
}

void FBodyInstance::SetShapeCollisionEnabled(const int32 ShapeIndex, ECollisionEnabled::Type NewType, bool bUpdatePhysicsFilterData)
{
	if (ensureAlways(BodySetup.IsValid()))
	{
		const ECollisionEnabled::Type OldType = GetShapeCollisionEnabled(ShapeIndex);
		if (OldType != NewType)
		{
			// If ShapeCollisionEnabled wasn't set up yet, copy values from BodySetup into it
			if (!bShapeCollisionEnabledIsSet)
			{
				const int32 ShapeCount = GetBodySetup()->AggGeom.GetElementCount();
				ShapeCollisionEnabled.Reset();
				bShapeCollisionEnabledIsSet = true;
				ShapeCollisionEnabled.SetNum(ShapeCount);
				for (int32 OptionalShapeIndex = 0; OptionalShapeIndex < ShapeCount; ++OptionalShapeIndex)
				{
					ShapeCollisionEnabled[OptionalShapeIndex] = GetBodySetup()->AggGeom.GetElement(OptionalShapeIndex)->GetCollisionEnabled();
				}
			}
			ShapeCollisionEnabled[ShapeIndex] = NewType;

			if (bUpdatePhysicsFilterData)
			{
				UpdatePhysicsFilterData();
			}
		}
	}
}

EDOFMode::Type FBodyInstance::ResolveDOFMode(EDOFMode::Type DOFMode)
{
	EDOFMode::Type ResultDOF = DOFMode;
	if (DOFMode == EDOFMode::Default)
	{
		ESettingsDOF::Type SettingDefaultPlane = UPhysicsSettings::Get()->DefaultDegreesOfFreedom;
		if (SettingDefaultPlane == ESettingsDOF::XYPlane) ResultDOF = EDOFMode::XYPlane;
		if (SettingDefaultPlane == ESettingsDOF::XZPlane) ResultDOF = EDOFMode::XZPlane;
		if (SettingDefaultPlane == ESettingsDOF::YZPlane) ResultDOF = EDOFMode::YZPlane;
		if (SettingDefaultPlane == ESettingsDOF::Full3D) ResultDOF  = EDOFMode::SixDOF;
	}

	return ResultDOF;
}

FVector FBodyInstance::GetLockedAxis() const
{
	EDOFMode::Type MyDOFMode = ResolveDOFMode(DOFMode);

	switch (MyDOFMode)
	{
	case EDOFMode::None: return FVector::ZeroVector;
	case EDOFMode::YZPlane: return FVector(1, 0, 0);
	case EDOFMode::XZPlane: return FVector(0, 1, 0);
	case EDOFMode::XYPlane: return FVector(0, 0, 1);
	case EDOFMode::CustomPlane: return CustomDOFPlaneNormal;
	case EDOFMode::SixDOF: return FVector::ZeroVector;
	default:	check(0);	//unsupported locked axis type
	}

	return FVector::ZeroVector;
}

void FBodyInstance::UseExternalCollisionProfile(UBodySetup* InExternalCollisionProfileBodySetup)
{
	ensureAlways(InExternalCollisionProfileBodySetup);
	ExternalCollisionProfileBodySetup = InExternalCollisionProfileBodySetup;
	bPendingCollisionProfileSetup = false;
	LoadProfileData(false);
}

void FBodyInstance::ClearExternalCollisionProfile()
{
	ExternalCollisionProfileBodySetup = nullptr;
	LoadProfileData(false);
}

void FBodyInstance::SetDOFLock(EDOFMode::Type NewAxisMode)
{
	DOFMode = NewAxisMode;

	CreateDOFLock();
}

void FBodyInstance::CreateDOFLock()
{
	if (DOFConstraint)
	{
		DOFConstraint->TermConstraint();
		FConstraintInstance::Free(DOFConstraint);
		DOFConstraint = NULL;
	}

	const FVector LockedAxis = GetLockedAxis();
	const EDOFMode::Type DOF = ResolveDOFMode(DOFMode);

	if (IsDynamic() == false || (LockedAxis.IsNearlyZero() && DOF != EDOFMode::SixDOF))
	{
		return;
	}

	//if we're using SixDOF make sure we have at least one constraint
	if (DOF == EDOFMode::SixDOF && !bLockXTranslation && !bLockYTranslation && !bLockZTranslation && !bLockXRotation && !bLockYRotation && !bLockZRotation)
	{
		return;
	}

	DOFConstraint = FConstraintInstance::Alloc();
	{
		DOFConstraint->ProfileInstance.ConeLimit.bSoftConstraint = false;
		DOFConstraint->ProfileInstance.TwistLimit.bSoftConstraint  = false;
		DOFConstraint->ProfileInstance.LinearLimit.bSoftConstraint  = false;

		const FTransform TM = GetUnrealWorldTransform(false);
		FVector Normal = FVector(1, 0, 0);
		FVector Sec = FVector(0, 1, 0);


		if(DOF != EDOFMode::SixDOF)
		{
			DOFConstraint->SetAngularSwing1Motion((bLockRotation || DOFMode != EDOFMode::CustomPlane) ? EAngularConstraintMotion::ACM_Locked : EAngularConstraintMotion::ACM_Free);
			DOFConstraint->SetAngularSwing2Motion((bLockRotation || DOFMode != EDOFMode::CustomPlane) ? EAngularConstraintMotion::ACM_Locked : EAngularConstraintMotion::ACM_Free);
			DOFConstraint->SetAngularTwistMotion(EAngularConstraintMotion::ACM_Free);
			
			DOFConstraint->SetLinearXMotion((bLockTranslation || DOFMode != EDOFMode::CustomPlane) ? ELinearConstraintMotion::LCM_Locked : ELinearConstraintMotion::LCM_Free);
			DOFConstraint->SetLinearYMotion(ELinearConstraintMotion::LCM_Free);
			DOFConstraint->SetLinearZMotion(ELinearConstraintMotion::LCM_Free);

			Normal = LockedAxis.GetSafeNormal();
			FVector Garbage;
			Normal.FindBestAxisVectors(Garbage, Sec);
		}
		else
		{
			DOFConstraint->SetAngularTwistMotion(bLockXRotation ? EAngularConstraintMotion::ACM_Locked : EAngularConstraintMotion::ACM_Free);
			DOFConstraint->SetAngularSwing2Motion(bLockYRotation ? EAngularConstraintMotion::ACM_Locked : EAngularConstraintMotion::ACM_Free);
			DOFConstraint->SetAngularSwing1Motion(bLockZRotation ? EAngularConstraintMotion::ACM_Locked : EAngularConstraintMotion::ACM_Free);

			DOFConstraint->SetLinearXMotion(bLockXTranslation ? ELinearConstraintMotion::LCM_Locked : ELinearConstraintMotion::LCM_Free);
			DOFConstraint->SetLinearYMotion(bLockYTranslation ? ELinearConstraintMotion::LCM_Locked : ELinearConstraintMotion::LCM_Free);
			DOFConstraint->SetLinearZMotion(bLockZTranslation ? ELinearConstraintMotion::LCM_Locked : ELinearConstraintMotion::LCM_Free);
		}

		DOFConstraint->PriAxis1 = TM.InverseTransformVectorNoScale(Normal);
		DOFConstraint->SecAxis1 = TM.InverseTransformVectorNoScale(Sec);

		DOFConstraint->PriAxis2 = Normal;
		DOFConstraint->SecAxis2 = Sec;
		DOFConstraint->Pos2 = TM.GetLocation();

		// Create constraint instance based on DOF
		DOFConstraint->InitConstraint(this, nullptr, 1.f, OwnerComponent.Get());
	}
}

ECollisionEnabled::Type FBodyInstance::GetCollisionEnabled_CheckOwner() const
{
	// Check actor override
	const UPrimitiveComponent* OwnerComponentInst = OwnerComponent.Get();
	AActor* Owner = OwnerComponentInst ? OwnerComponentInst->GetOwner() : nullptr;
	if (Owner && !Owner->GetActorEnableCollision())
	{
		return ECollisionEnabled::NoCollision;
	}
	else if(const USkeletalMeshComponent* SkelMeshComp = Cast<USkeletalMeshComponent>(OwnerComponentInst))
	{
		// Check component override (skel mesh case)
		return SkelMeshComp->BodyInstance.CollisionEnabled;
	}
	else
	{
		return CollisionEnabled;
	}
}

ECollisionEnabled::Type FBodyInstance::GetShapeCollisionEnabled(const int32 ShapeIndex) const
{
	// If any runtime shape collision overrides have been set, return that.
	// Otherwise, get it from the bodysetup.
	if (bShapeCollisionEnabledIsSet)
	{
		if (ensure(ShapeCollisionEnabled.IsValidIndex(ShapeIndex)))
		{
			return ShapeCollisionEnabled[ShapeIndex];
		}
	}

	if (!ensureAlways(BodySetup.IsValid()))
	{
		return ECollisionEnabled::NoCollision;
	}	

	FKShapeElem* Shape = GetBodySetup()->AggGeom.GetElement(ShapeIndex);
	if (!ensure(Shape))
	{
		return ECollisionEnabled::NoCollision;
	}

	return Shape->GetCollisionEnabled();
}

void FBodyInstance::SetMaskFilter(FMaskFilter InMaskFilter)
{
	if (MaskFilter == InMaskFilter)
	{
		return;
	}

	FPhysicsCommand::ExecuteWrite(GetPhysicsActor(), [&](const FPhysicsActorHandle& Actor)
	{
		TArray<FPhysicsShapeHandle> Shapes;
		FPhysicsInterface::GetAllShapes_AssumedLocked(Actor, Shapes);

		for(FPhysicsShapeHandle& Shape : Shapes)
		{
			const FBodyInstance* BI = GetOriginalBodyInstance(Shape);

			if(BI == this)
				{
				FPhysicsCommand::ExecuteShapeWrite(this, Shape, [&](const FPhysicsShapeHandle& InnerShape)
					{
					FPhysicsInterface::SetMaskFilter(InnerShape, InMaskFilter);
					});
				}
			}
	});

	MaskFilter = InMaskFilter;
}

/** Update the filter data on the physics shapes, based on the owning component flags. */
void FBodyInstance::UpdatePhysicsFilterData()
{
	SCOPE_CYCLE_COUNTER(STAT_UpdatePhysFilter);

	if(WeldParent)
	{
		WeldParent->UpdatePhysicsFilterData();
		return;
	}

	// Do nothing if no physics actor
	if (!IsValidBodyInstance())
	{
		return;
	}

	// this can happen in landscape height field collision component
	if (!BodySetup.IsValid())
	{
		return;
	}

	FPhysicsCommand::ExecuteWrite(GetPhysicsActorWeldRoot(), [&](const FPhysicsActorHandle& Actor)
	{
		TArray<FPhysicsShapeHandle> AllShapes;
		const int32 NumSyncShapes = FPhysicsInterface::GetAllShapes_AssumedLocked(GetPhysicsActor(), AllShapes);
		const int32 NumTotalShapes = AllShapes.Num();
		// In skeletal case, collision enable/disable/movement should be overriden by mesh component
		FBodyCollisionData BodyCollisionData;
		BuildBodyFilterData(BodyCollisionData.CollisionFilterData);
		BuildBodyCollisionFlags(BodyCollisionData.CollisionFlags, GetCollisionEnabled(), BodySetup->GetCollisionTraceFlag() == CTF_UseComplexAsSimple);

		bool bUpdateMassProperties = false;

		// We use these to determine the original shape index of an element.
		// TODO: If we stored ShapeIndex in FKShapeElem this wouldn't be necessary.
		int32 ShapeIndexBase = 0;
		const FBodyInstance* PrevBI = this;

		for(int32 ShapeIndex = 0; ShapeIndex < NumTotalShapes; ++ShapeIndex)
		{
			FPhysicsShapeHandle& Shape = AllShapes[ShapeIndex];
			const FBodyInstance* BI = GetOriginalBodyInstance(Shape);

			if (BI != PrevBI)
			{
				ShapeIndexBase = ShapeIndex;
				PrevBI = BI;
			}
			const int32 SetupShapeIndex = ShapeIndex - ShapeIndexBase;

			// If the BodyInstance that owns this shape is not 'this' BodyInstance (ie in the case of welding)
			// we need to generate new filter data using the owning original instance (and its BodySetup) 
			FBodyCollisionData PerShapeCollisionData;
			if(BI != this)
			{
				BI->BuildBodyFilterData(PerShapeCollisionData.CollisionFilterData);
			}
			else
			{
				PerShapeCollisionData = BodyCollisionData;
			}

			const bool bInstanceComplexAsSimple = BI->BodySetup.IsValid() ? (BI->BodySetup->GetCollisionTraceFlag() == CTF_UseComplexAsSimple) : false;
			if (BI->BodySetup.IsValid() && SetupShapeIndex < BI->GetBodySetup()->AggGeom.GetElementCount())
			{
				// Get the shape's CollisionResponses
				BI->BuildBodyFilterData(PerShapeCollisionData.CollisionFilterData, SetupShapeIndex);

				// Get the shape's CollisionEnabled masked with the body's CollisionEnabled and compute the shape's collisionflags.
				const ECollisionEnabled::Type FilteredShapeCollision = CollisionEnabledIntersection(BI->GetCollisionEnabled(), BI->GetShapeCollisionEnabled(SetupShapeIndex));
				BuildBodyCollisionFlags(PerShapeCollisionData.CollisionFlags, FilteredShapeCollision, bInstanceComplexAsSimple);
			}
			else
			{
				// This case may occur for trimeshes, which do not have toggleable shape collision. The assumption is made that
				// trimesh (complex) shapes are always created after all of the simple shapes.
				BuildBodyCollisionFlags(PerShapeCollisionData.CollisionFlags, BI->GetCollisionEnabled(), bInstanceComplexAsSimple);
			}


			FPhysicsCommand::ExecuteShapeWrite(this, Shape, [&](const FPhysicsShapeHandle& InnerShape)
			{
				// See if we currently have sim collision
				const bool bWasSimulationShape = FPhysicsInterface::IsSimulationShape(InnerShape);
				const bool bSyncShape = ShapeIndex < NumSyncShapes;
				const bool bIsTrimesh = FPhysicsInterface::GetShapeType(InnerShape) == ECollisionShapeType::Trimesh;
				const bool bIsStatic = FPhysicsInterface::IsStatic(Actor);

				const FBodyCollisionFlags& CollisionFlags = PerShapeCollisionData.CollisionFlags;
				const FBodyCollisionFilterData& FilterData = PerShapeCollisionData.CollisionFilterData;
				const bool bNewQueryShape = CollisionFlags.bEnableQueryCollision && (!bIsStatic || bSyncShape);
				const bool bNewSimShape = bIsTrimesh ? CollisionFlags.bEnableSimCollisionComplex : CollisionFlags.bEnableSimCollisionSimple;
				const bool bNewProbeShape = CollisionFlags.bEnableProbeCollision;

				FPhysicsInterface::SetIsQueryShape(InnerShape, bNewQueryShape);
				FPhysicsInterface::SetIsSimulationShape(InnerShape, bNewSimShape);
				FPhysicsInterface::SetIsProbeShape(InnerShape, bNewProbeShape);

				// If we changed 'simulation collision' on a shape, we need to recalc mass properties
				if (bWasSimulationShape != bNewSimShape)
				{
					bUpdateMassProperties = true;
				}

				// Apply new collision settings to this shape
				const Chaos::Filter::FShapeFilterData& ShapeFilterData = bIsTrimesh ? FilterData.ComplexShapeFilterData : FilterData.SimpleShapeFilterData;
				FPhysicsInterface::SetShapeFilterData(InnerShape, ShapeFilterData);
				FPhysicsInterface::SetFilterInstanceData(InnerShape, FilterData.FilterInstanceData);
			});
		}

		if(bUpdateMassProperties)
		{
			UpdateMassProperties();
		}

		//If filtering changed we must update GT structure right away
		if (FPhysScene* PhysScene = GetPhysicsScene())
		{
			PhysScene->UpdateActorInAccelerationStructure(Actor);
		}
		// Always wake actors up when collision filters change
		FPhysicsInterface::WakeUp_AssumesLocked(Actor);
	});

	UpdateInterpolateWhenSubStepping();
}

UWorld* FBodyInstance::GetWorld() const
{
	const UPrimitiveComponent* OwnerComponentInst = OwnerComponent.Get();
	if (OwnerComponentInst)
	{
		return OwnerComponentInst->GetWorld();
	}
	UObject* BodyInstanceObject = BodyInstanceOwner ? BodyInstanceOwner->GetSourceObject() : nullptr;
	return BodyInstanceObject ? BodyInstanceObject->GetWorld() : nullptr;
};


TAutoConsoleVariable<int32> CDisableQueryOnlyActors(TEXT("p.DisableQueryOnlyActors"), 0, TEXT("If QueryOnly is used, actors are marked as simulation disabled. This is NOT compatible with origin shifting at the moment."));

#if USE_BODYINSTANCE_DEBUG_NAMES
TSharedPtr<TArray<ANSICHAR>> GetDebugDebugName(const UPrimitiveComponent* PrimitiveComp, const UObject* SourceObject, const UBodySetup* BodySetup, FString& DebugName)
{
	// Setup names
	// Make the debug name for this geometry...
	DebugName.Reset();
	TSharedPtr<TArray<ANSICHAR>> PhysXName;

	if (PrimitiveComp)
	{
		DebugName += FString::Printf(TEXT("%s %s "), *AActor::GetDebugName(PrimitiveComp->GetOwner()), *PrimitiveComp->GetPathName());
		AActor* PrimitiveCompOwner = PrimitiveComp->GetOwner();
		UPackage* ExternalPackage = PrimitiveCompOwner ? PrimitiveCompOwner->GetExternalPackage() : nullptr;
		if (ExternalPackage)
		{
			DebugName += FString::Printf(TEXT("%s "), *ExternalPackage->GetName());
		}
	}
	else if (SourceObject && SourceObject->GetOuter())
	{
		DebugName += FString::Printf(TEXT("%s %s "), *SourceObject->GetOuter()->GetName(), *SourceObject->GetName());
	}
	else if (SourceObject)
	{
		DebugName += FString::Printf(TEXT("%s "), *SourceObject->GetName());
	}

	if (BodySetup->BoneName != NAME_None)
	{
		DebugName += FString::Printf(TEXT("Bone: '%s' "), *BodySetup->BoneName.ToString());
	}

	// Convert to char* for PhysX
	PhysXName = MakeShareable(new TArray<ANSICHAR>(StringToArray<ANSICHAR>(*DebugName, DebugName.Len() + 1)));

	return PhysXName;
}
#endif

static void GetSimulatingAndBlendWeight(const USkeletalMeshComponent* SkelMeshComp, const UBodySetup* BodySetup, float& InstanceBlendWeight, bool& bInstanceSimulatePhysics)
{
	bool bEnableSim = false;
	if (SkelMeshComp)
	{
		if(CollisionEnabledHasPhysics(SkelMeshComp->BodyInstance.GetCollisionEnabled()))
		{
			if ((BodySetup->PhysicsType == PhysType_Simulated) || (BodySetup->PhysicsType == PhysType_Default))
			{
				bEnableSim = (SkelMeshComp && IsRunningDedicatedServer()) ? SkelMeshComp->bEnablePhysicsOnDedicatedServer : true;
				bEnableSim &= ((BodySetup->PhysicsType == PhysType_Simulated) || (SkelMeshComp->BodyInstance.bSimulatePhysics));	//if unfixed enable. If default look at parent
			}
		}
	}
	else
	{
		//not a skeletal mesh so don't bother with default and skeletal mesh component
		bEnableSim = BodySetup->PhysicsType == PhysType_Simulated;
	}

	if (bEnableSim)
	{
		// set simulate to true if using physics
		bInstanceSimulatePhysics = true;
		if (BodySetup->PhysicsType == PhysType_Simulated)
		{
			InstanceBlendWeight = 1.f;
		}
	}
	else
	{
		bInstanceSimulatePhysics = false;
		if (BodySetup->PhysicsType == PhysType_Simulated)
		{
			InstanceBlendWeight = 0.f;
		}
	}
}

namespace UE::Private
{
	static UObject* GetSourceObject(UPrimitiveComponent* InPrimitiveComp, IPhysicsBodyInstanceOwner* InBodyInstanceOwner)
	{
		return InPrimitiveComp ? InPrimitiveComp->GetOwner() : InBodyInstanceOwner ? InBodyInstanceOwner->GetSourceObject() : nullptr;
	}
}

namespace Chaos::Private
{
	void FInitBodiesHelperBaseInternal::SetBodyInstanceExternalCollisionProfileBodySetup(FBodyInstance* BodyInstance, UBodySetup* ExternalCollisionProfileBodySetup)
	{
		BodyInstance->ExternalCollisionProfileBodySetup = ExternalCollisionProfileBodySetup;
	}

	void FInitBodiesHelperBaseInternal::UpdateBodyInstanceSolverAsyncDeltaTime(FBodyInstance* BodyInstance)
	{
		BodyInstance->UpdateSolverAsyncDeltaTime();
	}
};

template<bool bCompileStatic, typename InAllocatorType>
FInitBodiesHelper<bCompileStatic, InAllocatorType>::FInitBodiesHelper(
	TArray<FBodyInstance*, InAllocatorType>& InBodies, TArray<FTransform, InAllocatorType>& InTransforms, UBodySetup* InBodySetup,
	UPrimitiveComponent* InPrimitiveComp, FPhysScene* InRBScene,
	const FInitBodySpawnParams& InSpawnParams, FPhysicsAggregateHandle InAggregate, IPhysicsBodyInstanceOwner* InBodyInstanceOwner)
	: FInitBodiesHelper<bCompileStatic, InAllocatorType>(InBodies, InTransforms, InBodySetup, InPrimitiveComp,
		UE::Private::GetSourceObject(InPrimitiveComp, InBodyInstanceOwner),
		InRBScene, InSpawnParams, InAggregate, InBodyInstanceOwner)
{
}

template<typename InAllocatorType>
TInitBodiesHelperBase<InAllocatorType>::TInitBodiesHelperBase(TArray<FBodyInstance*, InAllocatorType>& InBodies, TArray<FTransform, InAllocatorType>& InTransforms,
	class UBodySetup* InBodySetup, UPrimitiveComponent* InComponent, UObject* InSourceObject, FPhysScene* InRBScene,
	const FInitBodySpawnParams& InSpawnParams, FPhysicsAggregateHandle InAggregate, IPhysicsBodyInstanceOwner* InBodyInstanceOwner)
	: Bodies(InBodies)
	, Transforms(InTransforms)
	, BodySetup(InBodySetup)
	, PrimitiveComp(InComponent)
	, BodyInstanceOwner(InBodyInstanceOwner)
	, SourceObject(InSourceObject)
	, PhysScene(InRBScene)
	, Aggregate(InAggregate)
#if USE_BODYINSTANCE_DEBUG_NAMES
	, DebugName(new FString())
#endif
	, bInstanceSimulatePhysics(false)
	, InstanceBlendWeight(-1.f)
	, SkelMeshComp(nullptr)
	, SpawnParams(InSpawnParams)
	, DisableQueryOnlyActors(!!CDisableQueryOnlyActors.GetValueOnAnyThread())
{
#if USE_BODYINSTANCE_DEBUG_NAMES
	PhysXName = GetDebugDebugName(PrimitiveComp, SourceObject, BodySetup, *DebugName);
#endif
}

template<typename InAllocatorType>
TInitBodiesHelperBase<InAllocatorType>::TInitBodiesHelperBase(TArray<FBodyInstance*, InAllocatorType>& InBodies, TArray<FTransform, InAllocatorType>& InTransforms,
	class UBodySetup* InBodySetup, class UPrimitiveComponent* InPrimitiveComp, FPhysScene* InRBScene,
	const FInitBodySpawnParams& InSpawnParams, FPhysicsAggregateHandle InAggregate, IPhysicsBodyInstanceOwner* InBodyInstanceOwner)
	// Forward to the other constructor
	: TInitBodiesHelperBase<InAllocatorType>(InBodies, InTransforms, InBodySetup, InPrimitiveComp,
		UE::Private::GetSourceObject(InPrimitiveComp, InBodyInstanceOwner),
		InRBScene, InSpawnParams, InAggregate, InBodyInstanceOwner)
{
}

template<typename InAllocatorType>
void TInitBodiesHelperBase<InAllocatorType>::UpdateSimulatingAndBlendWeight()
{
	GetSimulatingAndBlendWeight(SkelMeshComp, BodySetup, InstanceBlendWeight, bInstanceSimulatePhysics);
}

// Return to actor ref
template<typename InAllocatorType>
void TInitBodiesHelperBase<InAllocatorType>::CreateActor_AssumesLocked(FBodyInstance* Instance, const FTransform& Transform) const
{
	SCOPE_CYCLE_COUNTER(STAT_CreatePhysicsActor);
	checkSlow(!FPhysicsInterface::IsValid(Instance->GetPhysicsActor()));
	const ECollisionEnabled::Type CollisionType = Instance->GetCollisionEnabled();

	FActorCreationParams ActorParams;
	ActorParams.InitialTM = Transform;
	ActorParams.bSimulatePhysics = Instance->ShouldInstanceSimulatingPhysics();
	ActorParams.bStartAwake = Instance->bStartAwake;
#if USE_BODYINSTANCE_DEBUG_NAMES
	ActorParams.DebugName = Instance->CharDebugName.IsValid() ? Instance->CharDebugName->GetData() : nullptr;
#endif
	ActorParams.bEnableGravity = Instance->bEnableGravity;
	ActorParams.bUpdateKinematicFromSimulation = Instance->bUpdateKinematicFromSimulation;
	ActorParams.bQueryOnly = CollisionType == ECollisionEnabled::QueryOnly;
	ActorParams.Scene = PhysScene;
	ActorParams.bStatic = IsStatic();

	FPhysicsActorHandle ActorHandle = nullptr;
	FPhysicsInterface::CreateActor(ActorParams, ActorHandle, PrimitiveComp);
	Instance->SetPhysicsActor(ActorHandle);

	if (!IsStatic())
	{
		FPhysicsInterface::SetCcdEnabled_AssumesLocked(ActorHandle, Instance->bUseCCD);
		FPhysicsInterface::SetMACDEnabled_AssumesLocked(ActorHandle, Instance->IsUsingMACD());
		FPhysicsInterface::SetPartialIslandSleepAllowed_AssumesLocked(ActorHandle, Instance->IsPartialIslandSleepAllowed());
		FPhysicsInterface::SetIsKinematic_AssumesLocked(ActorHandle, !Instance->ShouldInstanceSimulatingPhysics());

		FPhysicsInterface::SetMaxLinearVelocity_AssumesLocked(ActorHandle, FMath::Sqrt(TNumericLimits<float>::Max()));
		FPhysicsInterface::SetSmoothEdgeCollisionsEnabled_AssumesLocked(ActorHandle, Instance->bSmoothEdgeCollisions);
		FPhysicsInterface::SetInertiaConditioningEnabled_AssumesLocked(ActorHandle, Instance->IsInertiaConditioningEnabled());
		FPhysicsInterface::SetGyroscopicTorqueEnabled_AssumesLocked(ActorHandle, Instance->bGyroscopicTorqueEnabled);
		FPhysicsInterface::SetPositionSolverIterationCount_AssumesLocked(ActorHandle, Instance->GetPositionSolverIterationCount());
		FPhysicsInterface::SetVelocitySolverIterationCount_AssumesLocked(ActorHandle, Instance->GetVelocitySolverIterationCount());
		FPhysicsInterface::SetProjectionSolverIterationCount_AssumesLocked(ActorHandle, Instance->GetProjectionSolverIterationCount());
		FPhysicsInterface::SetGravityGroupIndex_AssumesLocked(ActorHandle, Instance->GravityGroupIndex);

		// Set sleep event notification
		FPhysicsInterface::SetSendsSleepNotifies_AssumesLocked(ActorHandle, Instance->bGenerateWakeEvents);
	}
}

template<typename InAllocatorType>
bool TInitBodiesHelperBase<InAllocatorType>::CreateShapes_AssumesLocked(FBodyInstance* Instance) const
{
	SCOPE_CYCLE_COUNTER(STAT_CreatePhysicsShapes);
	UPhysicalMaterial* SimplePhysMat = Instance->GetSimplePhysicalMaterial();
	TArray<UPhysicalMaterial*> ComplexPhysMats;
	TArray<FPhysicalMaterialMaskParams> ComplexPhysMatMasks;

	ComplexPhysMats = Instance->GetComplexPhysicalMaterials(ComplexPhysMatMasks);

	FBodyCollisionData BodyCollisionData;
	if (const FBodyInstancePhysicsCreationInputs* Cached = Instance->GetAsyncPhysicsCreationInputs())
	{
		BodyCollisionData = Cached->ResolvedCollisionData;
	}
	else
	{
		Instance->BuildBodyFilterData(BodyCollisionData.CollisionFilterData);
		FBodyInstance::BuildBodyCollisionFlags(BodyCollisionData.CollisionFlags, Instance->GetCollisionEnabled(), BodySetup->GetCollisionTraceFlag() == CTF_UseComplexAsSimple);
	}

	bool bInitFail = false;

	// #PHYS2 Call interface AddGeometry
	BodySetup->AddShapesToRigidActor_AssumesLocked(Instance, Instance->Scale3D, SimplePhysMat, ComplexPhysMats, ComplexPhysMatMasks, BodyCollisionData, FTransform::Identity);

	FPhysicsInterface::SetIgnoreAnalyticCollisions_AssumesLocked(Instance->GetPhysicsActor(), CVarIgnoreAnalyticCollisionsOverride.GetValueOnAnyThread() ? true : Instance->bIgnoreAnalyticCollisions);

	const int32 NumShapes = FPhysicsInterface::GetNumShapes(Instance->GetPhysicsActor());
	bInitFail |= NumShapes == 0;

	return bInitFail;
}

UBodySetup* FBodyInstance::GetBodySetup() const
{
	if (const FBodyInstancePhysicsCreationInputs* Cached = GetAsyncPhysicsCreationInputs())
	{
		return Cached->ResolvedBodySetup;
	}

	FGCScopeGuard Guard;
	if(UBodySetupCore* BodySetupCore = BodySetup.Get())
	{
		return CastChecked<UBodySetup>(BodySetupCore);
	}

	return nullptr;
}

template<typename InAllocatorType>
const FString& GetBodyInstanceDebugName(TInitBodiesHelperBase<InAllocatorType>& InitHelper)
{
	static FString NullName = "<NoName>";

#if USE_BODYINSTANCE_DEBUG_NAMES
	if (InitHelper.DebugName.IsValid())
	{
		return *InitHelper.DebugName.Get();
	}
#endif
	return NullName;
}

bool ValidateTransformScale(const FTransform& Transform, const FString& DebugName)
{
	if (Transform.GetScale3D().IsNearlyZero())
	{
		UE_LOGF(LogPhysics, Warning, "Initialising Body : Scale3D is (nearly) zero: %ls", *DebugName);
		return false;
	}

	return true;
}

bool ValidateTransformMirror(const FTransform& Transform, const FString& DebugName, bool bGenerateMirroredCollision, bool bGenerateNonMirroredCollision)
{
	// Check we support mirroring/non-mirroring
	const float TransformDet = Transform.GetDeterminant();
	if (TransformDet < 0.f && !bGenerateMirroredCollision)
	{
		UE_LOGF(LogPhysics, Warning, "Initialising Body : Body is mirrored but bGenerateMirroredCollision == false: %ls", *DebugName);
		return false;
	}

	if (TransformDet > 0.f && !bGenerateNonMirroredCollision)
	{
		UE_LOGF(LogPhysics, Warning, "Initialising Body : Body is not mirrored but bGenerateNonMirroredCollision == false: %ls", *DebugName);
		return false;
	}

	return true;
}

bool ValidateTransformNaN(const FTransform& Transform, const FString& DebugName, const FName& BoneName)
{
#if !(UE_BUILD_SHIPPING)
	if (Transform.ContainsNaN())
	{
		UE_LOGF(LogPhysics, Warning, "Initialising Body : Bad transform - %ls %ls\n%ls", *DebugName, *BoneName.ToString(), *Transform.ToString());
		return false;
	}
#endif

	return true;
}


// Takes actor ref arrays.
// #PHYS2 this used to return arrays of low-level physics bodies, which would be added to scene in InitBodies. Should it still do that, rather then later iterate over BodyInstances to get phys actor refs?
template<typename InAllocatorType>
bool TInitBodiesHelperBase<InAllocatorType>::CreateShapesAndActors()
{
	SCOPE_CYCLE_COUNTER(STAT_CreatePhysicsShapesAndActors);
	TRACE_CPUPROFILER_EVENT_SCOPE(CreatePhysicsShapesAndActors);

	const int32 NumBodies = Bodies.Num();

	const bool bIsInGameThread = IsInGameThread();
	// Current case where CreateShapesAndActors is not called from the GameThread is when p.Chaos.EnableAsyncInitBody is true
	check(bIsInGameThread || (Chaos::CVars::bEnableAsyncInitBody && (BodySetup->bCreatedPhysicsMeshes || BodySetup->bNeverNeedsCookedCollisionData)));
	if (bIsInGameThread)
	{
		// Ensure we have the AggGeom inside the body setup so we can calculate the number of shapes
		BodySetup->CreatePhysicsMeshes();
	}

	if (bSkipShapeCreationForEmptyBodySetup)
	{
		if (BodySetup->TriMeshGeometries.IsEmpty() && BodySetup->AggGeom.GetElementCount() == 0)
		{
#if WITH_EDITOR
			// In the editor we may have ended up here because of world trace ignoring our EnableCollision.
			// Since we can't get at the data in that function we check for it here
			if (PrimitiveComp && PrimitiveComp->IsCollisionEnabled())
#endif
			{
				UE_LOGF(LogPhysics, Log, "Init of %d instances of Primitive Component %ls failed. Does it have collision data available?",
					NumBodies, PrimitiveComp ? *PrimitiveComp->GetReadableName() : TEXT("null"));
			}

			return false;
		}
	}

	for (int32 BodyIdx = NumBodies - 1; BodyIdx >= 0; BodyIdx--)   // iterate in reverse since list might shrink
	{
		FBodyInstance* Instance = Bodies[BodyIdx];
		const FTransform& Transform = Transforms[BodyIdx];

		// Log some warnings for unexpected transforms, but treat NaNs as errors
		const FString& SafeDebugName = GetBodyInstanceDebugName(*this);
		ValidateTransformScale(Transform, SafeDebugName);
		ValidateTransformMirror(Transform, SafeDebugName, BodySetup->bGenerateMirroredCollision, BodySetup->bGenerateNonMirroredCollision);
		const bool bValidTransform = ValidateTransformNaN(Transform, SafeDebugName, BodySetup->BoneName);
		if (!bValidTransform)
		{
			if (bPreventInvalidBodyInstanceTransforms)
			{
				// NaNs are errors and we don't create the physics state
				UE_LOGF(LogPhysics, Error, "Rejecting BodyInstance %d on %ls with an invalid transform", BodyIdx , *SafeDebugName);
				return false;
			}
			else
			{
				// NaNs are errors but we still create the physics state. This will almost certainly cause problems later
				UE_LOGF(LogPhysics, Error, "Creating a BodyInstance %d on %ls with an invalid transform which will likely lead to severe performance and behavioural problems in Physics", BodyIdx , *SafeDebugName);
			}
		}

		Instance->OwnerComponent = PrimitiveComp;
		Instance->BodyInstanceOwner = BodyInstanceOwner;
		Instance->SourceObject = SourceObject;
		Instance->BodySetup = BodySetup;
		Instance->Scale3D = Transform.GetScale3D();
#if USE_BODYINSTANCE_DEBUG_NAMES
		Instance->CharDebugName = PhysXName;
#endif
		Instance->bEnableGravity = Instance->bEnableGravity && (SkelMeshComp ? SkelMeshComp->BodyInstance.bEnableGravity : true);	//In the case of skeletal mesh component we AND bodies with the parent body

		// Handle autowelding here to avoid extra work
		if (!IsStatic() && Instance->bAutoWeld)
		{
			ECollisionEnabled::Type CollisionType = Instance->GetCollisionEnabled();
			if (CollisionType != ECollisionEnabled::QueryOnly)
			{
				if (UPrimitiveComponent * ParentPrimComponent = PrimitiveComp ? Cast<UPrimitiveComponent>(PrimitiveComp->GetAttachParent()) : NULL)
				{
					UWorld* World = PrimitiveComp->GetWorld();
					if (World && World->IsGameWorld())
					{
						check(bIsInGameThread);
						//if we have a parent we will now do the weld and exit any further initialization
						if (PrimitiveComp->WeldToImplementation(ParentPrimComponent, PrimitiveComp->GetAttachSocketName(), false))	//welded new simulated body so initialization is done
						{
							return false;
						}
					}
				}
			}
		}

		// Don't process if we've already got a body
		// Just ask actorref
		if (FPhysicsInterface::IsValid(Instance->GetPhysicsActor()))
		{
			Instance->OwnerComponent = nullptr;
			Instance->BodyInstanceOwner = nullptr;
			Instance->SourceObject = nullptr;
			Instance->BodySetup = nullptr;
			constexpr uint32 Count = 1;
			Bodies.RemoveAt(BodyIdx, Count, EAllowShrinking::No);  // so we wont add it to the physx scene again later.
			Transforms.RemoveAt(BodyIdx, Count, EAllowShrinking::No);
			continue;
		}

		// Set sim parameters for bodies from skeletal mesh components
		if (!IsStatic() && SpawnParams.bPhysicsTypeDeterminesSimulation)
		{
			Instance->bSimulatePhysics = bInstanceSimulatePhysics;
			if (InstanceBlendWeight != -1.0f)
			{
				Instance->PhysicsBlendWeight = InstanceBlendWeight;
			}
		}

		// Init user data structure to point back at this instance
		Instance->PhysicsUserData = FPhysicsUserData(Instance);

		CreateActor_AssumesLocked(Instance, Transform);
		const bool bInitFail = CreateShapes_AssumesLocked(Instance);
		if (bInitFail)
		{
#if WITH_EDITOR
			//In the editor we may have ended up here because of world trace ignoring our EnableCollision. Since we can't get at the data in that function we check for it here
			if(!PrimitiveComp || PrimitiveComp->IsCollisionEnabled())
#endif
			{
				UE_LOGF(LogPhysics, Log, "Init Instance %d of Primitive Component %ls failed. Does it have collision data available?", BodyIdx, *PrimitiveComp->GetReadableName());
			}

			Instance->ReleasePhysicsActor();

			Instance->OwnerComponent = nullptr;
			Instance->BodyInstanceOwner = nullptr;
			Instance->SourceObject = nullptr;
			Instance->BodySetup = nullptr;
			SetBodyInstanceExternalCollisionProfileBodySetup(Instance, nullptr);

			continue;
		}

		FPhysicsInterface::SetActorUserData_AssumesLocked(Instance->GetPhysicsActor(), &Instance->PhysicsUserData);

#if USE_BODYINSTANCE_DEBUG_NAMES
		Instance->GetPhysicsActor()->GetParticle_LowLevel()->SetDebugName(DebugName);
#endif
	}
	return true;
}

template<typename InAllocatorType>
void TInitBodiesHelperBase<InAllocatorType>::InitBodies()
{
	LLM_SCOPE(ELLMTag::ChaosBody);

	const bool bIsInGameThread = IsInGameThread();
	check(Chaos::CVars::bEnableAsyncInitBody || bIsInGameThread);
	if (bIsInGameThread ? CreateShapesAndActors() : true)
	{
		FPhysicsCommand::ExecuteWrite(PhysScene, [&]()
		{
			if (!bIsInGameThread && !CreateShapesAndActors())
			{
				return;
			}
			// If an aggregate present, add to that
			if (Aggregate.IsValid())
			{
				SCOPE_CYCLE_COUNTER(STAT_InitBodyAggregate);
				for (FBodyInstance* BI : Bodies)
				{
					FPhysicsActorHandle ActorHandle = BI->GetPhysicsActor();
					if (FPhysicsInterface::IsValid(ActorHandle))
					{
						FPhysicsInterface::AddActorToAggregate_AssumesLocked(Aggregate, ActorHandle);
					}
				}
			}
			else if (PhysScene)
			{
				SCOPE_CYCLE_COUNTER(STAT_InitBodyAdd);
				TArray<FPhysicsActorHandle> ActorHandles;
				ActorHandles.Reserve(Bodies.Num());

				for (FBodyInstance* BI : Bodies)
				{
					FPhysicsActorHandle ActorHandle = BI->GetPhysicsActor();
					if (FPhysicsInterface::IsValid(ActorHandle))
					{
						ActorHandles.Add(ActorHandle);

						Chaos::FRigidBodyHandle_External& Body_External = ActorHandle->GetGameThreadAPI();
						const int32 NumShapes = FPhysicsInterface::GetNumShapes(ActorHandle);

						// If this shape shouldn't collide in the sim we disable it here until we support
						// a separation of unions for these shapes
						if(BI->GetCollisionEnabled() == ECollisionEnabled::QueryOnly || BI->GetCollisionEnabled() == ECollisionEnabled::NoCollision)
						{
							for(int32 ShapeIndex = 0; ShapeIndex < NumShapes; ++ShapeIndex)
							{
								Body_External.SetShapeSimCollisionEnabled(ShapeIndex, false);
							}
						}
						if (const UBodySetup* BS = BI->GetBodySetup())
						{
							for (int32 ShapeIndex = 0; ShapeIndex < NumShapes; ++ShapeIndex)
							{
								Body_External.SetShapeCollisionTraceType(ShapeIndex, ChaosInterface::ConvertCollisionTraceFlag(BS->GetCollisionTraceFlag())) ;
							}
						}
					}
				}

				PhysScene->AddActorsToScene_AssumesLocked(ActorHandles);

				for (FBodyInstance* BI : Bodies)
				{
					FPhysicsActorHandle ActorHandle = BI->GetPhysicsActor();
					if (FPhysicsInterface::IsValid(ActorHandle))
					{
						// Resolve component pointer safely: cached map key on worker thread, weak ptr on game thread
						const FBodyInstancePhysicsCreationInputs* Cached = BI->GetAsyncPhysicsCreationInputs();
						if (UPrimitiveComponent* CompKey = Cached ? Cached->OwnerComponentMapKey : BI->OwnerComponent.Get())
						{
							PhysScene->AddToComponentMaps(CompKey, ActorHandle);

							if (BI->bNotifyRigidBodyCollision)
							{
								PhysScene->RegisterForCollisionEvents(CompKey);
							}
						}
					}
				}
			}

			// set solver async delta time if any bodies are overriding the sim delta time
			for (FBodyInstance* BI : Bodies)
			{				
				UpdateBodyInstanceSolverAsyncDeltaTime(BI);
			}

			// Set up dynamic instance data
			if (!IsStatic())
			{
				SCOPE_CYCLE_COUNTER(STAT_InitBodyPostAdd);
				for (int32 BodyIdx = 0, NumBodies = Bodies.Num(); BodyIdx < NumBodies; ++BodyIdx)
				{
					FBodyInstance* Instance = Bodies[BodyIdx];
					Instance->InitDynamicProperties_AssumesLocked();
				}
			}
		});
	}
}

// Explicit template instantiations must appear after all member function definitions (e.g., InitBodies)
// so that Clang emits every method. Placing them earlier causes linker failures on non-MSVC platforms
// because the compiler hasn't seen the method bodies yet at the point of instantiation.
template struct TInitBodiesHelperBase<FDefaultAllocator>;
template struct FInitBodiesHelper<true>;
template struct FInitBodiesHelper<false>;
using BodyInstanceSingleElementArrayInlineAllocator = TInlineAllocator<1>;
template struct TInitBodiesHelperBase<BodyInstanceSingleElementArrayInlineAllocator>;
template struct FInitBodiesHelper<true, BodyInstanceSingleElementArrayInlineAllocator>;
template struct FInitBodiesHelper<false, BodyInstanceSingleElementArrayInlineAllocator>;
using FInitSingleBodyHelperStatic = FInitBodiesHelper<true, BodyInstanceSingleElementArrayInlineAllocator>;
using FInitSingleBodyHelperDynamic = FInitBodiesHelper<false, BodyInstanceSingleElementArrayInlineAllocator>;

FInitBodySpawnParams::FInitBodySpawnParams(const UPrimitiveComponent* PrimComp)
{
	bStaticPhysics = PrimComp == nullptr || PrimComp->IsStaticPhysics();

	if(const USkeletalMeshComponent* SKOwner = Cast<USkeletalMeshComponent>(PrimComp))
	{
		bPhysicsTypeDeterminesSimulation = true;
	}
	else
	{
		bPhysicsTypeDeterminesSimulation = false;
	}
}

FInitBodySpawnParams::FInitBodySpawnParams(bool bInStaticPhysics, bool bInPhysicsTypeDeterminesSimulation)
	: bStaticPhysics(bInStaticPhysics)
	, bPhysicsTypeDeterminesSimulation(bInPhysicsTypeDeterminesSimulation)
{
}


// Chaos addition
static TAutoConsoleVariable<int32> CVarAllowCreatePhysxBodies(
	TEXT("p.chaos.AllowCreatePhysxBodies"),
	1,
	TEXT("")
	TEXT(" 0 is off, 1 is on (default)"),
	ECVF_ReadOnly);


void FBodyInstance::InitBody(UBodySetup* Setup, const FTransform& Transform, UPrimitiveComponent* PrimComp, FPhysScene* InRBScene, const FInitBodySpawnParams& SpawnParams, IPhysicsBodyInstanceOwner* InBodyInstanceOwner)
{
	InitBody(Setup, Transform, PrimComp, nullptr, InRBScene, SpawnParams, InBodyInstanceOwner);
}

void FBodyInstance::InitBody(UBodySetup* Setup, const FTransform& Transform, UPrimitiveComponent* PrimComp, UObject* InSourceObject, FPhysScene* InRBScene, const FInitBodySpawnParams& SpawnParams, IPhysicsBodyInstanceOwner* InBodyInstanceOwner)
{
	if (CVarAllowCreatePhysxBodies.GetValueOnAnyThread() == 0)
	{
		return;
	}

	SCOPE_CYCLE_COUNTER(STAT_InitBody);
	check(Setup);
	
	TArray<FBodyInstance*, BodyInstanceSingleElementArrayInlineAllocator> Bodies;
	TArray<FTransform, BodyInstanceSingleElementArrayInlineAllocator> Transforms;

	Bodies.Add(this);
	Transforms.Add(Transform);

	UObject* SourceObjectToUse = InSourceObject ? InSourceObject : UE::Private::GetSourceObject(PrimComp, InBodyInstanceOwner);
	bool bIsStatic = SpawnParams.bStaticPhysics;
	if(bIsStatic)
	{
		FInitSingleBodyHelperStatic InitBodyHelper(Bodies, Transforms, Setup, PrimComp, SourceObjectToUse, InRBScene, SpawnParams, SpawnParams.Aggregate, InBodyInstanceOwner);
		InitBodyHelper.InitBodies();
	}
	else
	{
		FInitSingleBodyHelperDynamic InitBodyHelper(Bodies, Transforms, Setup, PrimComp, SourceObjectToUse, InRBScene, SpawnParams, SpawnParams.Aggregate, InBodyInstanceOwner);
		InitBodyHelper.InitBodies();
	}

	UpdateInterpolateWhenSubStepping();
}

FVector GetInitialLinearVelocity(const AActor* OwningActor, bool& bComponentAwake)
{
	FVector InitialLinVel(EForceInit::ForceInitToZero);
	if (OwningActor)
	{
		InitialLinVel = OwningActor->GetVelocity();

		if (InitialLinVel.SizeSquared() > FMath::Square(UE_KINDA_SMALL_NUMBER))
		{
			bComponentAwake = true;
		}
	}

	return InitialLinVel;
}


const FBodyInstance* FBodyInstance::GetOriginalBodyInstance(const FPhysicsShapeHandle& InShape) const
{
	const FBodyInstance* BI = WeldParent ? WeldParent : this;
	const FWeldInfo* Result = BI->ShapeToBodiesMap.IsValid() ? BI->ShapeToBodiesMap->Find(InShape) : nullptr;
	return Result ? Result->ChildBI : BI;
}

const FTransform& FBodyInstance::GetRelativeBodyTransform(const FPhysicsShapeHandle& InShape) const
{
	check(IsInGameThread());
	const FBodyInstance* BI = WeldParent ? WeldParent : this;
	const FWeldInfo* Result = BI->ShapeToBodiesMap.IsValid() ? BI->ShapeToBodiesMap->Find(InShape) : nullptr;
	return Result ? Result->RelativeTM : FTransform::Identity;
}

FBodyInstance::FAsyncTermBodyPayload FBodyInstance::StartAsyncTermBody_GameThread()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FBodyInstance::StartAsyncTermBody_GameThread);
	SCOPE_CYCLE_COUNTER(STAT_TermBody);

	// Capture the actor handle before nulling it - AsyncTermBody will release it on the worker thread.
	FAsyncTermBodyPayload Payload;
	Payload.ActorHandle = GetPhysicsActor();

	// Null the actor so TermBody skips RemoveFromComponentMaps + ReleaseActor
	// (those will be done async by AsyncTermBody using the captured handle).
	SetPhysicsActor(nullptr);

	// TermBody handles everything else on GT: UnRegister, cleanup, DOFConstraint, per-shape overrides.
	TermBody();

	return Payload;
}

void FBodyInstance::AsyncTermBody(FAsyncTermBodyPayload& Payload)
{
	SCOPE_CYCLE_COUNTER(STAT_TermBody);

	if (FPhysicsInterface::IsValid(Payload.ActorHandle))
	{
		FPhysScene* PhysScene = static_cast<FPhysScene*>(FPhysicsInterface::GetCurrentScene(Payload.ActorHandle));
		if (PhysScene)
		{
			PhysScene->RemoveFromComponentMaps(Payload.ActorHandle);
		}
		// ReleaseActor handles null PhysScene (falls back to direct delete)
		FPhysicsInterface::ReleaseActor(Payload.ActorHandle, PhysScene);
	}

	check(!FPhysicsInterface::IsValid(Payload.ActorHandle));
}

/**
 *	Clean up the physics engine info for this instance.
 *	Can be called from a worker thread (see header comment for contract).
 */
void FBodyInstance::TermBody(bool bNeverDeferRelease)
{
	SCOPE_CYCLE_COUNTER(STAT_TermBody);

	const bool bIsGameThread = IsInGameThread();

	// Welded bodies check - only on game thread.
	// ShapeToBodiesMap is a non-thread-safe TSharedPtr and the error message dereferences OwnerComponent.
	if (bIsGameThread && ShapeToBodiesMap != nullptr && !ShapeToBodiesMap->IsEmpty())
	{
		TStringBuilder<256> Builder;
		int32 BodyIndex = 0;
		for (const TPair<FPhysicsShapeHandle, FWeldInfo>& It : *ShapeToBodiesMap)
		{
			const FBodyInstance* ChildBody = It.Value.ChildBI;
			const FString Name = ChildBody != nullptr ? ChildBody->GetBodyDebugName() : FString("null");
			Builder.Appendf(TEXT("\tBody[%d]: \'%s'\n"), BodyIndex, *Name);
			++BodyIndex;
		}
		ensureMsgf(false, TEXT("Body %s is being terminated but has %d welded bodies:\n%s"), *GetNameSafe(OwnerComponent.Get()), ShapeToBodiesMap->Num(), Builder.ToString());
	}

	// UnRegisterForCollisionEvents - game thread only.
	// Worker-thread callers must UnRegister on GT before going async (see header comment for contract).
	FPhysScene_Chaos* PhysScene = nullptr;
	if (bIsGameThread)
	{
		if (UPrimitiveComponent* PrimComp = OwnerComponent.Get())
		{
			PhysScene = PrimComp->GetWorld()->GetPhysicsScene();
			if (bNotifyRigidBodyCollision && PhysScene)
			{
				PhysScene->UnRegisterForCollisionEvents(PrimComp);
			}
		}
	}

	if (IsValidBodyInstance())
	{
		// Off game thread: get PhysScene from the actor handle (no UObject access needed)
		if (!PhysScene)
		{
			PhysScene = static_cast<FPhysScene_Chaos*>(FPhysicsInterface::GetCurrentScene(GetPhysicsActor()));
		}
		if (PhysScene)
		{
			PhysScene->RemoveFromComponentMaps(GetPhysicsActor());
		}
		ReleasePhysicsActor();
	}

	// @TODO: Release spring body here

	CurrentSceneState = BodyInstanceSceneState::NotAdded;
	BodySetup = NULL;
	OwnerComponent = NULL;
	BodyInstanceOwner = nullptr;
	ExternalCollisionProfileBodySetup = nullptr;

	// Reset per-shape runtime overrides
	if(bClearPerShapeOverridesOnTerminate)
	{
		bShapeCollisionEnabledIsSet = false;
		bShapeCollisionResponsesIsSet = false;
		ShapeCollisionEnabled.Reset();
		ShapeCollisionResponses.Reset();
	}

	if (DOFConstraint)
	{
		DOFConstraint->TermConstraint();
		FConstraintInstance::Free(DOFConstraint);
		DOFConstraint = NULL;
	}
}

bool FBodyInstance::Weld(FBodyInstance* TheirBody, const FTransform& TheirTM)
{
	check(IsInGameThread());
	check(TheirBody);
	if (TheirBody->BodySetup.IsValid() == false)	//attach actor can be called before body has been initialized. In this case just return false
	{
		return false;
	}

    if (TheirBody->WeldParent == this) // The body is already welded to this component. Do nothing.
    {
        return false;
    }

	TArray<FPhysicsShapeHandle> PNewShapes;

	FTransform MyTM = GetUnrealWorldTransform(false);
	MyTM.SetScale3D(Scale3D);	//physx doesn't store 3d so set it here

	FTransform RelativeTM = TheirTM.GetRelativeTransform(MyTM);

	FPhysicsCommand::ExecuteWrite(GetPhysicsActor(), [&](const FPhysicsActorHandle& Actor)
	{
		SCOPE_CYCLE_COUNTER(STAT_UpdatePhysMats);

		TheirBody->WeldParent = this;

		UPhysicalMaterial* SimplePhysMat = TheirBody->GetSimplePhysicalMaterial();

		TArray<UPhysicalMaterial*> ComplexPhysMats;
		TArray<FPhysicalMaterialMaskParams> ComplexPhysMatMasks;
	
		ComplexPhysMats = TheirBody->GetComplexPhysicalMaterials(ComplexPhysMatMasks);

		// This builds collision data based on this (parent) body, not their body. This gets fixed  up later though when PostShapeChange() calls UpdatePhysicsFilterData().
		FBodyCollisionData BodyCollisionData;
		BuildBodyFilterData(BodyCollisionData.CollisionFilterData);
		BuildBodyCollisionFlags(BodyCollisionData.CollisionFlags, GetCollisionEnabled(), BodySetup->GetCollisionTraceFlag() == CTF_UseComplexAsSimple);

		TheirBody->GetBodySetup()->AddShapesToRigidActor_AssumesLocked(this, Scale3D, SimplePhysMat, ComplexPhysMats, ComplexPhysMatMasks, BodyCollisionData, RelativeTM, &PNewShapes);

		FPhysicsInterface::SetSendsSleepNotifies_AssumesLocked(Actor, TheirBody->bGenerateWakeEvents);

		if(PNewShapes.Num())
		{
			if(!ShapeToBodiesMap.IsValid())
			{
				ShapeToBodiesMap = TSharedPtr<TMap<FPhysicsShapeHandle, FWeldInfo>>(new TMap<FPhysicsShapeHandle, FWeldInfo>());
			}

			for (int32 ShapeIdx = 0; ShapeIdx < PNewShapes.Num(); ++ShapeIdx)
			{
				PNewShapes[ShapeIdx].Shape->SetProxy(ActorHandle);
				ShapeToBodiesMap->Add(PNewShapes[ShapeIdx], FWeldInfo(TheirBody, RelativeTM));
			}

			if(TheirBody->ShapeToBodiesMap.IsValid())
			{
				TSet<FBodyInstance*> Bodies;
				//If the body that is welding to us has things welded to it, make sure to weld those things to us as well
				TMap<FPhysicsShapeHandle, FWeldInfo>& TheirWeldInfo = *TheirBody->ShapeToBodiesMap.Get();
				for(auto Itr = TheirWeldInfo.CreateIterator(); Itr; ++Itr)
				{
					const FWeldInfo& WeldInfo = Itr->Value;
					if(!Bodies.Contains(WeldInfo.ChildBI))
					{
						Bodies.Add(WeldInfo.ChildBI);	//only want to weld once per body and can have multiple shapes
						const FTransform ChildWorldTM = WeldInfo.RelativeTM * TheirTM;
						Weld(WeldInfo.ChildBI, ChildWorldTM);
					}
				}

				TheirWeldInfo.Empty();	//They are no longer root so empty this
			}
		}

		PostShapeChange();

		// remove their body from scenes (don't call TermBody because we don't want to clear things like BodySetup)
		FPhysicsInterface::ReleaseActor(TheirBody->ActorHandle, TheirBody->GetPhysicsScene());
	});
	
	UpdateInterpolateWhenSubStepping();

	TheirBody->UpdateDebugRendering();
	UpdateDebugRendering();

	return true;
}

int32 EnsureUnweldModifiesGTOnly = 0;
FAutoConsoleVariableRef CVarEnsureUnweldModifiesGTOnly(TEXT("p.EnsureUnweldModifiesGTOnly"), EnsureUnweldModifiesGTOnly, TEXT("Ensure if unweld modifies geometry shared with physics thread"));

void FBodyInstance::UnWeld(FBodyInstance* TheirBI)
{
	check(IsInGameThread());

	bool bShapesChanged = false;

	FPhysicsCommand::ExecuteWrite(GetPhysicsActor(), [&](const FPhysicsActorHandle& Actor)
				{
		TArray<FPhysicsShapeHandle> Shapes;
		const int32 NumSyncShapes = GetAllShapes_AssumesLocked(Shapes);
		const int32 NumTotalShapes = Shapes.Num();

		if(EnsureUnweldModifiesGTOnly && Actor->GetSolverBase() != nullptr)
		{
			ensureAlwaysMsgf(false, TEXT("Tried to unweld on body already in solver %s"), *GetBodyDebugName());
		}

		// reversed since FPhysicsInterface::DetachShape is removing shapes
		for (int Idx = Shapes.Num()-1; Idx >=0; Idx--)
		{
			FPhysicsShapeHandle& Shape = Shapes[Idx];
			const FBodyInstance* BI = GetOriginalBodyInstance(Shape);
			if (TheirBI == BI)
			{
				ShapeToBodiesMap->Remove(Shape);
				FPhysicsInterface::DetachShape(Actor, Shape);
				bShapesChanged = true;
			}
		}

	if (bShapesChanged)
	{
		PostShapeChange();
	}

		TheirBI->WeldParent = nullptr;
	});

	UpdateInterpolateWhenSubStepping();

	TheirBI->UpdateDebugRendering();
	UpdateDebugRendering();
}

void FBodyInstance::PostShapeChange()
{
	// Set the filter data on the shapes (call this after setting BodyData because it uses that pointer)
	UpdatePhysicsFilterData();

	UpdateMassProperties();
	// Update damping
	UpdateDampingProperties();
}

float AdjustForSmallThreshold(float NewVal, float OldVal)
{
	float Threshold = 0.1f;
	float Delta = NewVal - OldVal;
	if (Delta < 0 && FMath::Abs(NewVal) < Threshold)	//getting smaller and passed threshold so flip sign
	{
		return -Threshold;
	}
	else if (Delta > 0 && FMath::Abs(NewVal) < Threshold)	//getting bigger and passed small threshold so flip sign
	{
		return Threshold;
	}

	return NewVal;
}

//Non uniform scaling depends on the primitive that has the least non uniform scaling capability. So for example, a capsule's x and y axes scale are locked.
//So if a capsule exists in this body we must use locked x and y scaling for all shapes.
namespace EScaleMode
{
	enum Type
	{
		Free,
		LockedXY,
		LockedXYZ
	};
}

//computes the relative scaling vectors based on scale mode used
void ComputeScalingVectors(EScaleMode::Type ScaleMode, const FVector& InScale3D, FVector& OutScale3D, FVector& OutScale3DAbs)
{
	// Ensure no zeroes in any dimension
	FVector NewScale3D;
	NewScale3D.X = FMath::IsNearlyZero(InScale3D.X) ? UE_KINDA_SMALL_NUMBER : InScale3D.X;
	NewScale3D.Y = FMath::IsNearlyZero(InScale3D.Y) ? UE_KINDA_SMALL_NUMBER : InScale3D.Y;
	NewScale3D.Z = FMath::IsNearlyZero(InScale3D.Z) ? UE_KINDA_SMALL_NUMBER : InScale3D.Z;

	const FVector NewScale3DAbs = NewScale3D.GetAbs();
	switch (ScaleMode)
	{
	case EScaleMode::Free:
	{
		OutScale3D = NewScale3D;
		break;
	}
	case EScaleMode::LockedXY:
	{
		float XYScaleAbs = FMath::Max(NewScale3DAbs.X, NewScale3DAbs.Y);
		float XYScale = FMath::Max(NewScale3D.X, NewScale3D.Y) < 0.f ? -XYScaleAbs : XYScaleAbs;	//if both xy are negative we should make the xy scale negative

		OutScale3D = NewScale3D;
		OutScale3D.X = OutScale3D.Y = XYScale;

		break;
	}
	case EScaleMode::LockedXYZ:
	{
		float UniformScaleAbs = NewScale3DAbs.GetMin();	//uniform scale uses the smallest magnitude
		float UniformScale = FMath::Max3(NewScale3D.X, NewScale3D.Y, NewScale3D.Z) < 0.f ? -UniformScaleAbs : UniformScaleAbs;	//if all three values are negative we should make uniform scale negative

		OutScale3D = FVector(UniformScale);
		break;
	}
	default:
	{
		check(false);	//invalid scale mode
	}
	}

	OutScale3DAbs = OutScale3D.GetAbs();
}

EScaleMode::Type ComputeScaleMode(const TArray<FPhysicsShapeHandle>& Shapes)
{
	EScaleMode::Type ScaleMode = EScaleMode::Free;

	for(int32 ShapeIdx = 0; ShapeIdx < Shapes.Num(); ++ShapeIdx)
	{
		const FPhysicsShapeHandle& Shape = Shapes[ShapeIdx];
		ECollisionShapeType GeomType = FPhysicsInterface::GetShapeType(Shape);

		if(GeomType == ECollisionShapeType::Sphere)
		{
			ScaleMode = EScaleMode::LockedXYZ;	//sphere is most restrictive so we can stop
			break;
		}
		else if(GeomType == ECollisionShapeType::Capsule)
		{
			ScaleMode = EScaleMode::LockedXY;
		}
	}

	return ScaleMode;
}

void FBodyInstance::SetMassOverride(float MassInKG, bool bNewOverrideMass)
{
	bOverrideMass = bNewOverrideMass;
	MassInKgOverride = MassInKG;
}

bool FBodyInstance::GetRigidBodyState(FRigidBodyState& OutState)
{
	if (IsInstanceSimulatingPhysics())
	{
		FTransform BodyTM = GetUnrealWorldTransform();
		OutState.Position = BodyTM.GetTranslation();
		OutState.Quaternion = BodyTM.GetRotation();
		OutState.LinVel = GetUnrealWorldVelocity();
		OutState.AngVel = FMath::RadiansToDegrees(GetUnrealWorldAngularVelocityInRadians());
		OutState.Flags = (IsInstanceAwake() ? ERigidBodyFlags::None : ERigidBodyFlags::Sleeping);
		return true;
	}

	return false;
}

bool FBodyInstance::UpdateBodyScale(const FVector& InScale3D, bool bForceUpdate)
{
	using namespace Chaos;

	if (!IsValidBodyInstance())
	{
		//UE_LOGF(LogPhysics, Log, "Body hasn't been initialized. Call InitBody to initialize.");
		return false;
	}

	// if scale is already correct, and not forcing an update, do nothing
	if (Scale3D.Equals(InScale3D) && !bForceUpdate)
	{
		return false;
	}

	SCOPE_CYCLE_COUNTER(STAT_BodyInstanceUpdateBodyScale);

	bool bSuccess = false;

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	ensureMsgf ( !Scale3D.ContainsNaN() && !InScale3D.ContainsNaN(), TEXT("Scale3D = (%f,%f,%f) InScale3D = (%f,%f,%f)"), Scale3D.X, Scale3D.Y, Scale3D.Z, InScale3D.X, InScale3D.Y, InScale3D.Z );
#endif

	FVector UpdatedScale3D;

	//Get all shapes
	EScaleMode::Type ScaleMode = EScaleMode::Free;

	FPhysicsCommand::ExecuteWrite(GetPhysicsActor(), [&](const FPhysicsActorHandle& Actor)
	{
		TArray<FPhysicsShapeHandle> Shapes;
		GetAllShapes_AssumesLocked(Shapes);
		ScaleMode = ComputeScaleMode(Shapes);

		FVector AdjustedScale3D;
		FVector AdjustedScale3DAbs;

		// Apply scaling
		ComputeScalingVectors(ScaleMode, InScale3D, AdjustedScale3D, AdjustedScale3DAbs);

		UpdatedScale3D = AdjustedScale3D;

		TArray<Chaos::FImplicitObjectPtr> NewGeometry;
		NewGeometry.Reserve(Shapes.Num());


		for (FPhysicsShapeHandle& ShapeHandle : Shapes)
		{
			const Chaos::FImplicitObject& ImplicitObject = ShapeHandle.GetGeometry();
			
			EImplicitObjectType OuterType = ImplicitObject.GetType();
			EImplicitObjectType WrappedOrConcreteType = GetInnerType(OuterType);
			
			const FTransform& RelativeTM = GetRelativeBodyTransform(ShapeHandle);

			bool bIsTransformed = false;
			bool bIsScaled = false;
			bool bIsInstanced = false;

			// Unwrap the shape in order:
			// Transformed -> Scaled | Instanced -> Concrete

			if(OuterType == ImplicitObjectType::Transformed)
			{
				bIsTransformed = true;

				// Get GeomType that is transformed
				WrappedOrConcreteType = static_cast<const TImplicitObjectTransformed<FReal, 3>&>(ImplicitObject).GetTransformedObject()->GetType();
			}
			else
			{
				// If we didn't find a wrapper (transformed) then our wrapped or concrete type is the outer type
				WrappedOrConcreteType = OuterType;
			}

			// Strip out the scaled and instance wrappers leaving us with a definite concrete type
			EImplicitObjectType ConcreteType = WrappedOrConcreteType;

			if(IsScaled(ConcreteType))
			{
				bIsScaled = true;

				ConcreteType ^= ImplicitObjectType::IsScaled;
				ensure(!IsInstanced(ConcreteType));
			}
			else if(IsInstanced(ConcreteType))
			{
				bIsInstanced = true;

				ConcreteType ^= ImplicitObjectType::IsInstanced;
				ensure(!IsScaled(ConcreteType));
			}

			FKShapeElem* ShapeElem = FChaosUserData::Get<FKShapeElem>(FPhysicsInterface::GetUserData(ShapeHandle));

			switch(ConcreteType)
			{
				case ImplicitObjectType::Sphere:
				{
					if (!ShapeElem || !CHAOS_ENSURE(!bIsInstanced && !bIsScaled))
					{
						// No support for Instanced, Scaled not supported as we bake the scale below
						break;
					}

					FKSphereElem* SphereElem = ShapeElem->GetShapeCheck<FKSphereElem>();
					ensure(ScaleMode == EScaleMode::LockedXYZ);

					FReal Radius = FMath::Max<FReal>(SphereElem->Radius * AdjustedScale3DAbs.X, FCollisionShape::MinSphereRadius());
					FVec3 Center = RelativeTM.TransformPosition(SphereElem->Center) * InScale3D;
					Chaos::FImplicitObjectPtr NewSphere = MakeImplicitObjectPtr<Chaos::FSphere>(Center, Radius);

					NewGeometry.Emplace(MoveTemp(NewSphere));
					bSuccess = true;

					break;
				}
				case ImplicitObjectType::Box:
				{
					if (!ShapeElem || !CHAOS_ENSURE(!bIsScaled && !bIsInstanced))
					{
						// No support for ScaledImplicit Box yet or Instanced, scale is baked below
						break;
					}

					FKBoxElem* BoxElem = ShapeElem->GetShapeCheck<FKBoxElem>();

					const TBox<FReal, 3> * BoxGeometry = static_cast<const TBox<FReal, 3>*>(&ImplicitObject);

					FVec3 HalfExtents;
					HalfExtents.X = FMath::Max<FReal>((0.5f * BoxElem->X * AdjustedScale3DAbs.X), FCollisionShape::MinBoxExtent());
					HalfExtents.Y = FMath::Max<FReal>((0.5f * BoxElem->Y * AdjustedScale3DAbs.Y), FCollisionShape::MinBoxExtent());
					HalfExtents.Z = FMath::Max<FReal>((0.5f * BoxElem->Z * AdjustedScale3DAbs.Z), FCollisionShape::MinBoxExtent());

					FRigidTransform3 LocalTransform = BoxElem->GetTransform() * RelativeTM;
					LocalTransform.ScaleTranslation(AdjustedScale3D);

					// If not already transformed and has a rotation, must convert to transformed geometry.
					if (bIsTransformed || (LocalTransform.GetRotation() == FQuat::Identity))
					{
						// Center at origin, transform holds translation
						const FVec3 Min = -HalfExtents;
						const FVec3 Max =  HalfExtents;

						Chaos::FImplicitObjectPtr NewBox = MakeImplicitObjectPtr<TBox<FReal, 3>>(Min, Max);
						Chaos::FImplicitObjectPtr NewTransformedBox = MakeImplicitObjectPtr<TImplicitObjectTransformed<FReal, 3>>(MoveTemp(NewBox), LocalTransform);
						NewGeometry.Emplace(MoveTemp(NewTransformedBox));
					}
					else
					{
						// Bake in transformed position
						const FVec3 Min = LocalTransform.GetLocation() - HalfExtents;
						const FVec3 Max = LocalTransform.GetLocation() + HalfExtents;

						Chaos::FImplicitObjectPtr NewBox = MakeImplicitObjectPtr<TBox<FReal, 3>>(Min, Max);
						NewGeometry.Emplace(MoveTemp(NewBox));
					}

					bSuccess = true;

					break;
				}
				case ImplicitObjectType::Capsule:
				{
					if (!ShapeElem || !CHAOS_ENSURE(!bIsInstanced && !bIsScaled))
					{
						// No support for Instanced
						break;
					}

					ensure(ScaleMode == EScaleMode::LockedXY || ScaleMode == EScaleMode::LockedXYZ);

					FReal ScaleRadius = FMath::Max(AdjustedScale3DAbs.X, AdjustedScale3DAbs.Y);
					FReal ScaleLength = AdjustedScale3DAbs.Z;

					FReal Radius = 0.0f;
					FReal HalfHeight = 0.0f;
					FVec3 Center;
					FVec3 Axis;

					if (ShapeElem->GetShapeType() == EAggCollisionShape::TaperedCapsule)
					{
						// Handle the case where standard capsules are generated in place of tapered capsules, which are not fully supported.
						FKTaperedCapsuleElem* TaperedCapsuleElem = ShapeElem->GetShapeCheck<FKTaperedCapsuleElem>();

						const FReal MeanRadius = 0.5f * (TaperedCapsuleElem->Radius0 + TaperedCapsuleElem->Radius1);

						const FReal InitialHeight = MeanRadius * 2.0f + TaperedCapsuleElem->Length;
						Radius = FMath::Max(MeanRadius * ScaleRadius, (FReal)0.1);
						HalfHeight = (TaperedCapsuleElem->Length * 0.5f + MeanRadius) * ScaleLength;

						// TODO: For Transformed implicit, do not bake this in. Set Transform instead.
						Center = RelativeTM.TransformPosition(TaperedCapsuleElem->Center) * InScale3D;
						Axis = TaperedCapsuleElem->Rotation.RotateVector(Chaos::FVec3(0, 0, 1));
					}
					else
					{
						FKSphylElem* SphylElem = ShapeElem->GetShapeCheck<FKSphylElem>();

						const FReal InitialHeight = SphylElem->Radius * 2.0f + SphylElem->Length;
						Radius = FMath::Max(SphylElem->Radius * ScaleRadius, (FReal)0.1);
						HalfHeight = (SphylElem->Length * 0.5f + SphylElem->Radius) * ScaleLength;

						// TODO: For Transformed implicit, do not bake this in. Set Transform instead.
						Center = RelativeTM.TransformPosition(SphylElem->Center) * InScale3D;
						Axis = SphylElem->Rotation.RotateVector(Chaos::FVec3(0, 0, 1));
					}

					Radius = FMath::Min(Radius, HalfHeight);	//radius is capped by half length
					Radius = FMath::Max(Radius, (FReal)FCollisionShape::MinCapsuleRadius());
					FReal HalfLength = HalfHeight - Radius;
					HalfLength = FMath::Max((FReal)FCollisionShape::MinCapsuleAxisHalfHeight(), HalfLength);

					const FVec3 X1 = Center - HalfLength * Axis;
					const FVec3 X2 = Center + HalfLength * Axis;

					Chaos::FImplicitObjectPtr NewCapsule =  MakeImplicitObjectPtr<FCapsule>(X1, X2, Radius);
					NewGeometry.Emplace(MoveTemp(NewCapsule));

					bSuccess = true;

					break;
				}
				case ImplicitObjectType::Convex:
				{
					if (!ShapeElem || !CHAOS_ENSURE(bIsInstanced || bIsScaled))
					{
						// Expecting instanced or scaled.
						break;
					}

					FKConvexElem* ConvexElem = ShapeElem->GetShapeCheck<FKConvexElem>();
					const Chaos::FConvexPtr& ConvexImplicit = ConvexElem->GetChaosConvexMesh();

					Chaos::FImplicitObjectPtr NewConvex = nullptr;
					if (AdjustedScale3D == FVector(1.0f, 1.0f, 1.0f))
					{
						NewConvex = MakeImplicitObjectPtr<TImplicitObjectInstanced<FConvex>>(ConvexImplicit);
					}
					else
					{
						NewConvex = MakeImplicitObjectPtr<TImplicitObjectScaled<FConvex>>(ConvexImplicit, AdjustedScale3D);
					}

					if(RelativeTM.GetRotation() != FQuat::Identity || RelativeTM.GetTranslation() != FVector::ZeroVector)
					{
						FTransform AdjustedTransform = RelativeTM;
						AdjustedTransform.SetTranslation(RelativeTM.GetTranslation() * AdjustedScale3D);
						NewConvex = MakeImplicitObjectPtr<TImplicitObjectTransformed<FReal, 3>>(MoveTemp(NewConvex), AdjustedTransform);
					}
					
					NewGeometry.Emplace(MoveTemp(NewConvex));

					bSuccess = true;

					break;
				}
				case ImplicitObjectType::TriangleMesh:
				{
					if(!CHAOS_ENSURE(bIsScaled || bIsInstanced))
					{
						// Currently assuming all triangle meshes are scaled or instanced (if scale == 1).
						break;
					}

					auto CreateTriGeomAuto = [](auto InObject, TArray<Chaos::FImplicitObjectPtr>& OutGeoArray, const FVec3& InScale) -> Chaos::FImplicitObjectPtr
					{
						if(InScale == Chaos::FVec3(1.0f, 1.0f, 1.0f))
						{
							return MakeImplicitObjectPtr<Chaos::TImplicitObjectInstanced<Chaos::FTriangleMeshImplicitObject>>(InObject);
						}
						else
						{
							return MakeImplicitObjectPtr<Chaos::TImplicitObjectScaled<Chaos::FTriangleMeshImplicitObject>>(MoveTemp(InObject), InScale);
						}
					};

					const FImplicitObject* TrimeshContainer = &ImplicitObject;
					if(bIsTransformed)
					{
						TrimeshContainer = static_cast<const TImplicitObjectTransformed<FReal, 3>*>(TrimeshContainer)->GetTransformedObject();
					}

					FTriangleMeshImplicitObjectPtr InnerTriangleMesh = nullptr;
					if (bIsScaled)
					{
						const TImplicitObjectScaled<FTriangleMeshImplicitObject>* ScaledTriangleMesh = (static_cast<const TImplicitObjectScaled<FTriangleMeshImplicitObject>*>(TrimeshContainer));
						InnerTriangleMesh = ScaledTriangleMesh->Object();

						if(!InnerTriangleMesh)
						{
							// While a body setup will instantiate the triangle mesh as a shared geometry, other methods might not (e.g. retopologized landscape)
							NewGeometry.Emplace(MakeImplicitObjectPtr<Chaos::TImplicitObjectScaled<Chaos::FTriangleMeshImplicitObject>>(ScaledTriangleMesh->Object(), AdjustedScale3D));
						}
					}
					else if (bIsInstanced)
					{
						const TImplicitObjectInstanced<FTriangleMeshImplicitObject>* InstancedTriangleMesh = (static_cast<const TImplicitObjectInstanced<FTriangleMeshImplicitObject>*>(TrimeshContainer));
						InnerTriangleMesh = InstancedTriangleMesh->Object();
					}
					else
					{
						CHAOS_ENSURE(false);
						break;
					}

					if(InnerTriangleMesh)
					{
						Chaos::FImplicitObjectPtr NewTrimesh = CreateTriGeomAuto(MoveTempIfPossible(InnerTriangleMesh), NewGeometry, AdjustedScale3D);

						// If we have a transform - wrap the trimesh
						if(RelativeTM.GetRotation() != FQuat::Identity || RelativeTM.GetTranslation() != FVector::ZeroVector)
						{
							FTransform AdjustedTransform = RelativeTM;
							AdjustedTransform.SetTranslation(RelativeTM.GetTranslation() * AdjustedScale3D);
							NewTrimesh = MakeImplicitObjectPtr<TImplicitObjectTransformed<FReal, 3>>(MoveTemp(NewTrimesh), AdjustedTransform);
						}

						NewGeometry.Emplace(MoveTemp(NewTrimesh));
					}

					bSuccess = true;

					break;
				}
				case ImplicitObjectType::HeightField:
				{
					// HeightField is only used by Landscape, which does different code path from other primitives
					break;
				}
				default:
				{
					CHAOS_ENSURE(false);
					UE_LOGF(LogPhysics, Warning, "UpdateBodyScale: Unimplemented ImplicitObject of type: %d skipped.", OuterType);
				}
			}// end switch
		}

		// Many types not yet implemented for UpdateBodyScale. If any geometry is missing from array, we cannot update geometry without losing data.
		// Only follow through with update if all shapes succeeded.
		if (CHAOS_ENSURE(NewGeometry.Num() == Shapes.Num()))
		{
			GetPhysicsActor()->GetGameThreadAPI().SetGeometry(MakeImplicitObjectPtr<Chaos::FImplicitObjectUnion>(MoveTemp(NewGeometry)));
			FPhysicsInterface::WakeUp_AssumesLocked(GetPhysicsActor());
		}
		else
		{
			bSuccess = false;
		}
	});

	if (bSuccess)
	{
		Scale3D = UpdatedScale3D;

		if (FPhysScene_Chaos* Scene = GetPhysicsScene())
		{
			Scene->UpdateActorInAccelerationStructure(GetPhysicsActor());
		}

		// update mass if required
		if (bUpdateMassWhenScaleChanges)
		{
			bDirtyMassProps = true;

			//if already simulated compute mass immediately
			if (ShouldInstanceSimulatingPhysics())
			{
				UpdateMassProperties();
			}
		}
	}

	return bSuccess;
}

void FBodyInstance::UpdateInstanceSimulatePhysics()
{
	// In skeletal case, we need both our bone and skelcomponent flag to be true.
	// This might be 'and'ing us with ourself, but thats fine.
	const bool bUseSimulate = IsInstanceSimulatingPhysics();
	bool bInitialized = false;

	FPhysicsCommand::ExecuteWrite(GetPhysicsActor(), [&](const FPhysicsActorHandle& Actor)
	{
		bInitialized = true;
		FPhysicsInterface::SetIsKinematic_AssumesLocked(Actor, !bUseSimulate);
		FPhysicsInterface::SetCcdEnabled_AssumesLocked(Actor, bUseCCD);
		FPhysicsInterface::SetMACDEnabled_AssumesLocked(Actor, bUseMACD);
		FPhysicsInterface::SetPartialIslandSleepAllowed_AssumesLocked(Actor, bAllowPartialIslandSleep);
		FPhysicsInterface::SetOneWayInteraction_AssumesLocked(Actor, bOneWayInteraction);
		FPhysicsInterface::SetPositionSolverIterationCount_AssumesLocked(Actor, GetPositionSolverIterationCount());
		FPhysicsInterface::SetVelocitySolverIterationCount_AssumesLocked(Actor, GetVelocitySolverIterationCount());
		FPhysicsInterface::SetProjectionSolverIterationCount_AssumesLocked(Actor, GetProjectionSolverIterationCount());

		if(bSimulatePhysics)
		{
			if(bDirtyMassProps)
			{
				UpdateMassProperties();
			}

			if(bStartAwake)
			{
				FPhysicsInterface::WakeUp_AssumesLocked(Actor);
			}
		}
	});

	//In the original physx only implementation this was wrapped in a PRigidDynamic != NULL check.
	//We use bInitialized to check rigid actor has been created in either engine because if we haven't even initialized yet, we don't want to undo our settings
	if (bInitialized)
	{
		if (bUseSimulate)
		{
			PhysicsBlendWeight = 1.f;
		}
		else
		{
			PhysicsBlendWeight = 0.f;
		}

		bSimulatePhysics = bUseSimulate;
	}
}

bool FBodyInstance::IsNonKinematic() const
{
	return bSimulatePhysics;
}

bool FBodyInstance::IsDynamic() const
{
	return FPhysicsInterface::IsDynamic(GetPhysicsActor());
}

void FBodyInstance::ApplyWeldOnChildren()
{
	if(UPrimitiveComponent* OwnerComponentInst = OwnerComponent.Get())
	{
		TArray<FBodyInstance*> ChildrenBodies;
		TArray<FName> ChildrenLabels;
		OwnerComponentInst->GetWeldedBodies(ChildrenBodies, ChildrenLabels, /*bIncludingAutoWeld=*/true);

		for (int32 ChildIdx = 0; ChildIdx < ChildrenBodies.Num(); ++ChildIdx)
		{
			FBodyInstance* ChildBI = ChildrenBodies[ChildIdx];
			checkSlow(ChildBI);
			if (ChildBI != this)
			{
				const ECollisionEnabled::Type ChildCollision = ChildBI->GetCollisionEnabled();
				if(CollisionEnabledHasPhysics(ChildCollision))
				{
					if(UPrimitiveComponent* PrimOwnerComponent = ChildBI->OwnerComponent.Get())
					{
						Weld(ChildBI, PrimOwnerComponent->GetSocketTransform(ChildrenLabels[ChildIdx]));
					}
				}
			}
		}
	}
	
}

void FBodyInstance::SetInstanceSimulatePhysics(bool bSimulate, bool bMaintainPhysicsBlending, bool bPreserveExistingAttachment)
{
	if (bSimulate)
	{
		UPrimitiveComponent* OwnerComponentInst = OwnerComponent.Get();

		// If we are enabling simulation, and we are the root body of our component (or we are welded), we detach the component 
		if (OwnerComponentInst && OwnerComponentInst->IsRegistered() && (OwnerComponentInst->GetBodyInstance() == this || OwnerComponentInst->IsWelded()))
		{
			if (!bPreserveExistingAttachment && OwnerComponentInst->GetAttachParent())
			{
				OwnerComponentInst->DetachFromComponent(FDetachmentTransformRules::KeepWorldTransform);
			}

			if (bSimulatePhysics == false)	//if we're switching from kinematic to simulated
			{
				ApplyWeldOnChildren();
			}
		}

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
		if (OwnerComponentInst)
		{
			if (!IsValidBodyInstance())
			{
				FMessageLog("PIE").Warning(FText::Format(LOCTEXT("SimPhysNoBody", "Trying to simulate physics on ''{0}'' but no physics body."),
					FText::FromString(GetPathNameSafe(OwnerComponentInst))));
			}
			else if (FPhysicsInterface::IsStatic(GetPhysicsActor()))
			{
				FMessageLog("PIE").Warning(FText::Format(LOCTEXT("SimPhysStatic", "Trying to simulate physics on ''{0}'' but it is static."),
					FText::FromString(GetPathNameSafe(OwnerComponentInst))));
			}
			else if(BodySetup.IsValid() && BodySetup->GetCollisionTraceFlag() == ECollisionTraceFlag::CTF_UseComplexAsSimple)
			{
				FMessageLog("PIE").Warning(FText::Format(LOCTEXT("SimComplexAsSimple", "Trying to simulate physics on ''{0}'' but it has ComplexAsSimple collision."),
					FText::FromString(GetPathNameSafe(OwnerComponentInst))));
			}
		}
#endif
	}

	bSimulatePhysics = bSimulate;
	if ( !bMaintainPhysicsBlending )
	{
		if (bSimulatePhysics)
		{
			PhysicsBlendWeight = 1.f;
		}
		else
		{
			PhysicsBlendWeight = 0.f;
		}
	}

	UpdateInstanceSimulatePhysics();
}

bool FBodyInstance::IsValidBodyInstance() const
{
	return FPhysicsInterface::IsValid(GetPhysicsActor());
}

FTransform GetUnrealWorldTransformImp_AssumesLocked(const FBodyInstance* BodyInstance, bool bWithProjection, bool bGlobalPose)
{
	FTransform WorldTM = FTransform::Identity;

	if(BodyInstance && BodyInstance->IsValidBodyInstance())
	{
		WorldTM = FPhysicsInterface::GetTransform_AssumesLocked(BodyInstance->GetPhysicsActor(), bGlobalPose);

		if(bWithProjection)
		{
			BodyInstance->ExecuteOnCalculateCustomProjection(WorldTM);
		}
	}

	return WorldTM;
}

FTransform FBodyInstance::GetUnrealWorldTransform(bool bWithProjection /* = true*/, bool bForceGlobalPose /* = true*/) const
{
	FTransform OutTransform = FTransform::Identity;

	FPhysicsCommand::ExecuteRead(GetPhysicsActor(), [&](const FPhysicsActorHandle& Actor)
	{
		OutTransform = GetUnrealWorldTransformImp_AssumesLocked(this, bWithProjection, bForceGlobalPose);
	});

	return OutTransform;
}


FTransform FBodyInstance::GetUnrealWorldTransform_AssumesLocked(bool bWithProjection /* = true*/, bool bForceGlobalPose /* = true*/) const
{
	return GetUnrealWorldTransformImp_AssumesLocked(this, bWithProjection, bForceGlobalPose);
}

void FBodyInstance::SetBodyTransform(const FTransform& NewTransform, ETeleportType Teleport, bool bAutoWake)
{
	SCOPE_CYCLE_COUNTER(STAT_SetBodyTransform);

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	extern bool GShouldLogOutAFrameOfSetBodyTransform;
	if (GShouldLogOutAFrameOfSetBodyTransform == true)
	{
		UE_LOGF(LogPhysics, Log, "SetBodyTransform: %ls", *GetBodyDebugName());
	}
#endif // !(UE_BUILD_SHIPPING || UE_BUILD_TEST)

	// Catch NaNs and elegantly bail out.

	if( !ensureMsgf(!NewTransform.ContainsNaN(), TEXT("SetBodyTransform contains NaN (%s)\n%s"), (OwnerComponent.Get() ? *OwnerComponent->GetPathName() : TEXT("NONE")), *NewTransform.ToString()) )
	{
		return;
	}

	if(FPhysicsInterface::IsValid(GetPhysicsActor()))
	{
		if(!NewTransform.IsValid())
		{
			UE_LOGF(LogPhysics, Warning, "FBodyInstance::SetBodyTransform: Trying to set new transform with bad data: %ls", *NewTransform.ToString());
			return;
		}

		bool bEditorWorld = false;
#if WITH_EDITOR
		//If the body is moved in the editor we avoid setting the kinematic target. This is useful for tools that rely on the physx data being up to date in the editor (and velocities aren't important in this case)
		UPrimitiveComponent* OwnerComp = OwnerComponent.Get();
		UWorld* World = OwnerComp ? OwnerComp->GetWorld() : nullptr;
		bEditorWorld = World && World->WorldType == EWorldType::Editor;
#endif

		FPhysScene* Scene = GetPhysicsScene();

		if(FPhysicsInterface::IsDynamic(GetPhysicsActor()) && !bEditorWorld && Scene)
		{
			FPhysicsCommand::ExecuteWrite(GetPhysicsActor(), [&](const FPhysicsActorHandle& Actor)
			{
				const bool bKinematic = FPhysicsInterface::IsKinematic_AssumesLocked(Actor);
				const bool bSimulated = FPhysicsInterface::CanSimulate_AssumesLocked(Actor);
				const bool bIsSimKinematic = bKinematic && bSimulated;

				if(bIsSimKinematic && Teleport == ETeleportType::None)
				{
					Scene->SetKinematicTarget_AssumesLocked(this, NewTransform, true);
				}
				else
				{
					// todo(chaos): Calling SetKinematicTarget_AssumesLocked before SetGlobalPose_AssumesLocked is unnessary for chaos. We should fix this when PhysX is removed.
					if(bIsSimKinematic)
					{
						FPhysicsInterface::SetKinematicTarget_AssumesLocked(Actor, NewTransform);
					}

					FPhysicsInterface::SetGlobalPose_AssumesLocked(Actor, NewTransform);
				}
			});
		}
		else if(Scene)
		{
			FPhysicsCommand::ExecuteWrite(GetPhysicsActor(), [&](const FPhysicsActorHandle& Actor)
			{
				FPhysicsInterface::SetGlobalPose_AssumesLocked(Actor, NewTransform);
			});
		}
	}
	else if(WeldParent)
	{
		WeldParent->SetWeldedBodyTransform(this, NewTransform);
	}
}

void FBodyInstance::SetWeldedBodyTransform(FBodyInstance* TheirBody, const FTransform& NewTransform)
{
	UnWeld(TheirBody);
	Weld(TheirBody, NewTransform);
}

FVector FBodyInstance::GetUnrealWorldVelocity() const
{
	FVector OutVelocity = FVector::ZeroVector;

	FPhysicsCommand::ExecuteRead(GetPhysicsActor(), [&](const FPhysicsActorHandle& Actor)
	{
		OutVelocity = GetUnrealWorldVelocity_AssumesLocked();
	});

	return OutVelocity;
}

FTransform FBodyInstance::GetKinematicTarget_AssumesLocked() const
{
	FTransform TM;
	if (FPhysicsInterface::IsValid(GetPhysicsActor()))
	{
		TM = FPhysicsInterface::GetKinematicTarget_AssumesLocked(GetPhysicsActor());
	}
	return TM;
}

FTransform FBodyInstance::GetKinematicTarget() const
{
	FTransform TM;
	FPhysicsCommand::ExecuteRead(GetPhysicsActor(), [&](const FPhysicsActorHandle& Actor)
		{
			TM = FPhysicsInterface::GetKinematicTarget_AssumesLocked(Actor);
		});

	return TM;
}

FVector FBodyInstance::GetUnrealWorldVelocity_AssumesLocked() const
{
	FVector LinVel(EForceInit::ForceInitToZero);
	if (FPhysicsInterface::IsValid(GetPhysicsActor()))
	{
		LinVel = FPhysicsInterface::GetLinearVelocity_AssumesLocked(GetPhysicsActor());
	}

	return LinVel;
}

/** Note: returns angular velocity in radians per second. */
FVector FBodyInstance::GetUnrealWorldAngularVelocityInRadians() const
{
	FVector OutVelocity = FVector::ZeroVector;

	FPhysicsCommand::ExecuteRead(GetPhysicsActor(), [&](const FPhysicsActorHandle& Actor)
	{
		OutVelocity = FPhysicsInterface::GetAngularVelocity_AssumesLocked(Actor);
	});

	return OutVelocity;
}

/** Note: returns angular velocity in radians per second. */
FVector FBodyInstance::GetUnrealWorldAngularVelocityInRadians_AssumesLocked() const
{
	return FPhysicsInterface::GetAngularVelocity_AssumesLocked(GetPhysicsActor());
}

FVector FBodyInstance::GetUnrealWorldVelocityAtPoint(const FVector& Point) const
{
	FVector OutVelocity = FVector::ZeroVector;
	FPhysicsCommand::ExecuteRead(GetPhysicsActor(), [&](const FPhysicsActorHandle& Actor)
	{
		OutVelocity = FPhysicsInterface::GetWorldVelocityAtPoint_AssumesLocked(Actor, Point);
	});

	return OutVelocity;
}


FVector FBodyInstance::GetUnrealWorldVelocityAtPoint_AssumesLocked(const FVector& Point) const
{
	return FPhysicsInterface::GetWorldVelocityAtPoint_AssumesLocked(GetPhysicsActor(), Point);
}

FTransform FBodyInstance::GetMassSpaceToWorldSpace() const
{
	FTransform MassSpaceToWorldSpace = FTransform::Identity;

	FPhysicsCommand::ExecuteRead(GetPhysicsActor(), [&](const FPhysicsActorHandle& Actor)
		{
		MassSpaceToWorldSpace = FPhysicsInterface::GetComTransform_AssumesLocked(Actor);
	});

	return MassSpaceToWorldSpace;
}

FTransform FBodyInstance::GetMassSpaceLocal() const
{
	FTransform MassSpaceLocal = FTransform::Identity;

	FPhysicsCommand::ExecuteRead(GetPhysicsActor(), [&](const FPhysicsActorHandle& Actor)
	{
		MassSpaceLocal = FPhysicsInterface::GetComTransformLocal_AssumesLocked(Actor);
	});

	return MassSpaceLocal;
}

void FBodyInstance::SetMassSpaceLocal(const FTransform& NewMassSpaceLocalTM)
{
	//TODO: UE doesn't store this so any changes to mass properties will not remember about this properly
	FPhysicsCommand::ExecuteWrite(GetPhysicsActor(), [&](const FPhysicsActorHandle& Actor)
	{
		FPhysicsInterface::SetComLocalPose_AssumesLocked(Actor, NewMassSpaceLocalTM);
	});
}

float FBodyInstance::GetBodyMass() const
{
	float OutMass = 0.0f;

	FPhysicsCommand::ExecuteRead(GetPhysicsActor(), [&](const FPhysicsActorHandle& Actor)
	{
		OutMass = FPhysicsInterface::GetMass_AssumesLocked(Actor);
	});

	return OutMass;
}


FVector FBodyInstance::GetBodyInertiaTensor() const
{
	FVector OutTensor = FVector::ZeroVector;

	FPhysicsCommand::ExecuteRead(GetPhysicsActor(), [&](const FPhysicsActorHandle& Actor)
	{
		OutTensor = FPhysicsInterface::GetLocalInertiaTensor_AssumesLocked(Actor);
	});

	return OutTensor;
}

void FBodyInstance::SetInertiaConditioningEnabled(bool bInEnabled)
{
	if (bInEnabled != bInertiaConditioning)
	{
		bInertiaConditioning = bInEnabled;
		FPhysicsCommand::ExecuteWrite(GetPhysicsActor(),
			[bInEnabled](const FPhysicsActorHandle& Actor)
			{
				if (FPhysicsInterface::IsRigidBody(Actor))
				{
					FPhysicsInterface::SetInertiaConditioningEnabled_AssumesLocked(Actor, bInEnabled);
				}
			});
	}
}

FBox FBodyInstance::GetBodyBounds() const
{
	FBox OutBox(EForceInit::ForceInitToZero);

	FPhysicsCommand::ExecuteRead(GetPhysicsActor(), [&](const FPhysicsActorHandle& Actor)
		{
			OutBox = FPhysicsInterface::GetBounds_AssumesLocked(Actor);
		});

	return OutBox;
}

FBox FBodyInstance::GetBodyBoundsLocal() const
{
	FBox OutBox(EForceInit::ForceInitToZero);

	FPhysicsCommand::ExecuteRead(GetPhysicsActor(), [&](const FPhysicsActorHandle& Actor)
		{
			OutBox = FPhysicsInterface::GetBoundsLocal_AssumesLocked(Actor);
		});

	return OutBox;
}

void FBodyInstance::DrawCOMPosition(FPrimitiveDrawInterface* PDI, float COMRenderSize, const FColor& COMRenderColor)
{
	if (IsValidBodyInstance())
	{
		DrawWireStar(PDI, GetCOMPosition(), COMRenderSize, COMRenderColor, SDPG_Foreground);
	}
}

/** Utility for copying properties from one BodyInstance to another. */
void FBodyInstance::CopyBodyInstancePropertiesFrom(const FBodyInstance* FromInst)
{
	// No copying of runtime instances (strictly defaults off BodySetup)
	check(FromInst);
	check(FromInst->OwnerComponent.Get() == NULL);
	check(FromInst->BodySetup.Get() == NULL);
	check(!FPhysicsInterface::IsValid(FromInst->GetPhysicsActor()));
	check(!FPhysicsInterface::IsValid(GetPhysicsActor()));

	*this = *FromInst;
}

void FBodyInstance::CopyRuntimeBodyInstancePropertiesFrom(const FBodyInstance* FromInst)
{
	check(FromInst);
	check(!FromInst->bPendingCollisionProfileSetup);

	if (FromInst->bOverrideWalkableSlopeOnInstance)
	{
		SetWalkableSlopeOverride(FromInst->GetWalkableSlopeOverride());
	}

	CollisionResponses = FromInst->CollisionResponses;
	CollisionProfileName = FromInst->CollisionProfileName;
	CollisionEnabled = FromInst->CollisionEnabled;

	UpdatePhysicsFilterData();
}

const FPhysScene* FBodyInstance::GetPhysicsScene() const
{
	return static_cast<const FPhysScene*>(FPhysicsInterface::GetCurrentScene(GetPhysicsActor()));
}

FPhysScene* FBodyInstance::GetPhysicsScene()
{
	return static_cast<FPhysScene*>(FPhysicsInterface::GetCurrentScene(GetPhysicsActor()));
}

FPhysicsActorHandle& FBodyInstance::GetPhysicsActorHandle()
{
	return ActorHandle;
}

const FPhysicsActorHandle& FBodyInstance::GetPhysicsActorHandle() const
{
	return ActorHandle;
}

const FPhysicsActorHandle& FBodyInstance::GetActorReferenceWithWelding() const
{
	return WeldParent ? WeldParent->GetPhysicsActorHandle() : GetPhysicsActorHandle();
}

FPhysicsActorHandle FBodyInstance::GetPhysicsActor() const
{
	return ActorHandle;
}

FPhysicsActorHandle FBodyInstance::GetPhysicsActorWeldRoot() const
{
	return (WeldParent != nullptr) ? WeldParent->GetPhysicsActor() : GetPhysicsActor();
}

void FBodyInstance::SetPhysicsActor(FPhysicsActorHandle InHandle)
{
	ActorHandle = InHandle;
}

void FBodyInstance::ReleasePhysicsActor()
{
	if (ActorHandle != nullptr)
	{
		FPhysicsInterface::ReleaseActor(ActorHandle, GetPhysicsScene());
		check(ActorHandle == nullptr);
	}
}


const FWalkableSlopeOverride& FBodyInstance::GetWalkableSlopeOverride() const
{
	if (bOverrideWalkableSlopeOnInstance || !BodySetup.IsValid())
	{
		return WalkableSlopeOverride;
	}
	else
	{
		return GetBodySetup()->WalkableSlopeOverride;
	}
}

void FBodyInstance::SetWalkableSlopeOverride(const FWalkableSlopeOverride& NewOverride, bool bNewOverideSetting)
{
	bOverrideWalkableSlopeOnInstance = bNewOverideSetting;
	WalkableSlopeOverride = NewOverride;
}


bool FBodyInstance::GetOverrideWalkableSlopeOnInstance() const
{
	return bOverrideWalkableSlopeOnInstance;
}

/** 
*	Changes the current PhysMaterialOverride for this body. 
*	Note that if physics is already running on this component, this will _not_ alter its mass/inertia etc, it will only change its 
*	surface properties like friction and the damping.
*/
void FBodyInstance::SetPhysMaterialOverride( UPhysicalMaterial* NewPhysMaterial )
{
	// Save ref to PhysicalMaterial
	PhysMaterialOverride = NewPhysMaterial;

	// Go through the chain of physical materials and update the shapes 
	UpdatePhysicalMaterials();

	// Because physical material has changed, we need to update the mass
	UpdateMassProperties();
}

UPhysicalMaterial* FBodyInstance::GetSimplePhysicalMaterial() const
{
	return GetSimplePhysicalMaterial(this, OwnerComponent, GetBodySetup(), BodyInstanceOwner);
}

namespace UE::Private
{
	const IPhysicsBodyInstanceOwner* GetEffectiveBodyInstanceOwner(const TWeakObjectPtr<UPrimitiveComponent>& InOwnerComp, IPhysicsBodyInstanceOwner* InBodyInstanceOwner)
	{
		// Use provided body instance owner if valid, else use component
		const IPhysicsBodyInstanceOwner* EffectiveBodyInstanceOwner = InBodyInstanceOwner;
		if (!EffectiveBodyInstanceOwner)
		{
			if (UPrimitiveComponent* OwnerPrimComponent = InOwnerComp.Get())
			{
				EffectiveBodyInstanceOwner = OwnerPrimComponent;
				check(EffectiveBodyInstanceOwner);
			}
		}
		return EffectiveBodyInstanceOwner;
	}
}

UPhysicalMaterial* FBodyInstance::GetSimplePhysicalMaterial(const FBodyInstance* BodyInstance, TWeakObjectPtr<UPrimitiveComponent> OwnerComp, TWeakObjectPtr<UBodySetup> BodySetupPtr, IPhysicsBodyInstanceOwner* BodyInstanceOwner)
{
	if (!ensure(BodyInstance))
	{
		return nullptr;
	}

	if (const FBodyInstancePhysicsCreationInputs* Cached = BodyInstance->GetAsyncPhysicsCreationInputs())
	{
		return Cached->ResolvedSimpleMaterial;
	}

	if(!GEngine || !GEngine->DefaultPhysMaterial)
	{
		UE_LOGF(LogPhysics, Error, "FBodyInstance::GetSimplePhysicalMaterial : GEngine not initialized! Cannot call this during native CDO construction, wrap with if(!HasAnyFlags(RF_ClassDefaultObject)) or move out of constructor, material parameters will not be correct.");

		return nullptr;
	}

	// Find the PhysicalMaterial we need to apply to the physics bodies.
	// (LOW priority) Engine Mat, Material PhysMat, BodySetup Mat, Component/BodyInstanceOwner Override, Body Override (HIGH priority)
	
	UPhysicalMaterial* ReturnPhysMaterial = NULL;

	// BodyInstance override
	if (BodyInstance->PhysMaterialOverride != NULL)
	{
		ReturnPhysMaterial = BodyInstance->PhysMaterialOverride;
		check(!ReturnPhysMaterial || ReturnPhysMaterial->IsValidLowLevel());
	}
	else
	{
		// Component/BodyInstanceOwner override
		const IPhysicsBodyInstanceOwner* EffectiveBodyInstanceOwner = UE::Private::GetEffectiveBodyInstanceOwner(OwnerComp, BodyInstanceOwner);
		UPhysicalMaterial* BodyInstancePhysicalMaterialOverride = EffectiveBodyInstanceOwner ? EffectiveBodyInstanceOwner->GetPhysicsMaterialOverride() : nullptr;
		if (BodyInstancePhysicalMaterialOverride)
		{
			ReturnPhysMaterial = BodyInstancePhysicalMaterialOverride;
			check(!ReturnPhysMaterial || ReturnPhysMaterial->IsValidLowLevel());
		}
		else
		{
			// BodySetup
			UBodySetup* BodySetupRawPtr = BodySetupPtr.Get();
			if (BodySetupRawPtr && BodySetupRawPtr->PhysMaterial != NULL)
			{
				ReturnPhysMaterial = BodySetupPtr->PhysMaterial;
				check(!ReturnPhysMaterial || ReturnPhysMaterial->IsValidLowLevel());
			}
			else
			{
				// See if the Material has a PhysicalMaterial
				UPhysicalMaterial* PhysMatFromMaterial = NULL;
				if (UMaterialInterface* Material = EffectiveBodyInstanceOwner ? EffectiveBodyInstanceOwner->GetPhysicsMaterialBase() : nullptr)
				{
					PhysMatFromMaterial = Material->GetPhysicalMaterial();
				}

				if (PhysMatFromMaterial != NULL)
				{
					ReturnPhysMaterial = PhysMatFromMaterial;
					check(!ReturnPhysMaterial || ReturnPhysMaterial->IsValidLowLevel());
				}
				// fallback is default physical material
				else
				{
					ReturnPhysMaterial = GEngine->DefaultPhysMaterial;
					check(!ReturnPhysMaterial || ReturnPhysMaterial->IsValidLowLevel());
				}
			}
		}
	}
	
	return ReturnPhysMaterial;
}

TArray<UPhysicalMaterial*> FBodyInstance::GetComplexPhysicalMaterials() const
{
	TArray<UPhysicalMaterial*> PhysMaterials;
	GetComplexPhysicalMaterials(PhysMaterials);
	return PhysMaterials;
}

TArray<UPhysicalMaterial*> FBodyInstance::GetComplexPhysicalMaterials(TArray<FPhysicalMaterialMaskParams>& OutPhysMaterialMasks) const
{
	TArray<UPhysicalMaterial*> PhysMaterials;
	GetComplexPhysicalMaterials(PhysMaterials, OutPhysMaterialMasks);
	return PhysMaterials;
}

void FBodyInstance::GetComplexPhysicalMaterials(TArray<UPhysicalMaterial*>& OutPhysMaterials) const
{
	GetComplexPhysicalMaterials(this, OwnerComponent, OutPhysMaterials, nullptr, BodyInstanceOwner);
}

void FBodyInstance::GetComplexPhysicalMaterials(TArray<UPhysicalMaterial*>& OutPhysMaterials, TArray<FPhysicalMaterialMaskParams>& OutPhysMaterialMasks) const
{
	GetComplexPhysicalMaterials(this, OwnerComponent, OutPhysMaterials, &OutPhysMaterialMasks, BodyInstanceOwner);
}

void FBodyInstance::GetComplexPhysicalMaterials(const FBodyInstance* BodyInstance, TWeakObjectPtr<UPrimitiveComponent> OwnerComp, TArray<UPhysicalMaterial*>& OutPhysMaterials, TArray<FPhysicalMaterialMaskParams>* OutPhysMaterialMasks, IPhysicsBodyInstanceOwner* InBodyInstanceOwner)
{
	if (!ensure(BodyInstance))
	{
		return;
	}

	if (const FBodyInstancePhysicsCreationInputs* Cached = BodyInstance->GetAsyncPhysicsCreationInputs())
	{
		if (OutPhysMaterialMasks)
		{
			*OutPhysMaterialMasks = Cached->ResolvedComplexMasks;
		}
		OutPhysMaterials = Cached->ResolvedComplexMaterials;
		return;
	}

	if(!GEngine || !GEngine->DefaultPhysMaterial)
	{
		UE_LOGF(LogPhysics, Error, "FBodyInstance::GetComplexPhysicalMaterials : GEngine not initialized! Cannot call this during native CDO construction, wrap with if(!HasAnyFlags(RF_ClassDefaultObject)) or move out of constructor, material parameters will not be correct.");
		return;
	}

	// Find the PhysicalMaterial we need to apply to the physics bodies.
	// (LOW priority) Engine Mat, Material PhysMat, Component/BodyInstanceOwner Override, Body Override (HIGH priority)
	
	// BodyInstance override
	if (BodyInstance && BodyInstance->PhysMaterialOverride != nullptr)
	{
		OutPhysMaterials.SetNum(1);
		OutPhysMaterials[0] = BodyInstance->PhysMaterialOverride;
		check(!OutPhysMaterials[0] || OutPhysMaterials[0]->IsValidLowLevel());
	}
	else
	{
		// Component/BodyInstanceOwner override
		if (const IPhysicsBodyInstanceOwner* EffectiveBodyInstanceOwner = UE::Private::GetEffectiveBodyInstanceOwner(OwnerComp, InBodyInstanceOwner))
		{
			if (UPhysicalMaterial* BodyInstancePhysicalMaterialOverride = EffectiveBodyInstanceOwner->GetPhysicsMaterialOverride())
			{
				OutPhysMaterials.SetNum(1);
				OutPhysMaterials[0] = BodyInstancePhysicalMaterialOverride;
				check(!OutPhysMaterials[0] || OutPhysMaterials[0]->IsValidLowLevel());
			}
			else
			{
				// See if the Material has a PhysicalMaterial
				const int32 NumMaterials = EffectiveBodyInstanceOwner->GetNumMaterials();
				OutPhysMaterials.SetNum(NumMaterials);

				if (OutPhysMaterialMasks)
				{
					OutPhysMaterialMasks->SetNum(NumMaterials);
				}

				for (int32 MatIdx = 0; MatIdx < NumMaterials; MatIdx++)
				{
					UPhysicalMaterial* PhysMat = GEngine->DefaultPhysMaterial;
					UMaterialInterface* Material = EffectiveBodyInstanceOwner->GetMaterial(MatIdx);
					if (Material)
					{
						PhysMat = Material->GetPhysicalMaterial();
					}

					OutPhysMaterials[MatIdx] = PhysMat;

					if (OutPhysMaterialMasks)
					{
						UPhysicalMaterialMask* PhysMatMask = nullptr;
						UMaterialInterface* PhysMatMap = nullptr;

						if (Material)
						{
							PhysMatMask = Material->GetPhysicalMaterialMask();
							if (PhysMatMask)
							{
								PhysMatMap = Material;
							}
						}

						(*OutPhysMaterialMasks)[MatIdx].PhysicalMaterialMask = PhysMatMask;
						(*OutPhysMaterialMasks)[MatIdx].PhysicalMaterialMap = PhysMatMap;
					}
				}
			}
		}
	}
}

/** Util for finding the number of 'collision sim' shapes on this Actor */
int32 GetNumSimShapes_AssumesLocked(const FPhysicsActorHandle& ActorRef)
{
	FInlineShapeArray PShapes;
	const int32 NumShapes = FillInlineShapeArray_AssumesLocked(PShapes, ActorRef);

	int32 NumSimShapes = 0;

	for(FPhysicsShapeHandle& Shape : PShapes)
	{
		if(FPhysicsInterface::IsSimulationShape(Shape))
		{
			NumSimShapes++;
		}
	}

	return NumSimShapes;
}


void FBodyInstance::UpdateMassProperties()
{
	bDirtyMassProps = false;

	UPhysicalMaterial* PhysMat = GetSimplePhysicalMaterial();

	if (FPhysicsInterface::IsValid(GetPhysicsActor()) && FPhysicsInterface::IsRigidBody(GetPhysicsActor()))
	{
		FPhysicsCommand::ExecuteWrite(GetPhysicsActor(), [&](FPhysicsActorHandle Actor)
		{
			check(FPhysicsInterface::IsValid(Actor));

			if (GetNumSimShapes_AssumesLocked(Actor) > 0)
			{
				const int32 NumShapes = FPhysicsInterface::GetNumShapes(Actor);

				TArray<FPhysicsShapeHandle> Shapes;
				Shapes.AddUninitialized(NumShapes);
				FPhysicsInterface::GetAllShapes_AssumedLocked(Actor, Shapes);

				// Ignore trimeshes & shapes which don't contribute to the mass
				for (int32 ShapeIdx = Shapes.Num() - 1; ShapeIdx >= 0; --ShapeIdx)
				{
					const FPhysicsShapeHandle& Shape = Shapes[ShapeIdx];
					const FKShapeElem* ShapeElem = FChaosUserData::Get<FKShapeElem>(FPhysicsInterface::GetUserData(Shape));
					bool bIsTriangleMesh = FPhysicsInterface::GetShapeType(Shape) == ECollisionShapeType::Trimesh;
					bool bHasNoMass = ShapeElem && !ShapeElem->GetContributeToMass();
					if (bIsTriangleMesh || bHasNoMass)
					{
						Shapes.RemoveAtSwap(ShapeIdx);
					}
				}

				Chaos::FMassProperties TotalMassProperties;

				if (ShapeToBodiesMap.IsValid() && ShapeToBodiesMap->Num() > 0)
				{
					struct FWeldedBatch
					{
						TArray<FPhysicsShapeHandle> Shapes;
						FTransform RelTM;
					};

					// If we have welded children we must compute the mass properties of each individual body first and then combine them 
					// all together because each welded body may have different density etc
					TMap<FBodyInstance*, FWeldedBatch> BodyToShapes;

					for (const FPhysicsShapeHandle& Shape : Shapes) //sort all welded children by their original bodies
					{
						if (FWeldInfo* WeldInfo = ShapeToBodiesMap->Find(Shape))
						{
							FWeldedBatch* WeldedBatch = BodyToShapes.Find(WeldInfo->ChildBI);
							if (!WeldedBatch)
							{
								WeldedBatch = &BodyToShapes.Add(WeldInfo->ChildBI);
								WeldedBatch->RelTM = WeldInfo->RelativeTM;
							}

							WeldedBatch->Shapes.Add(Shape);
						}
						else
						{
							//no weld info so shape really belongs to this body
							FWeldedBatch* WeldedBatch = BodyToShapes.Find(this);
							if (!WeldedBatch)
							{
								WeldedBatch = &BodyToShapes.Add(this);
								WeldedBatch->RelTM = FTransform::Identity;
							}

							WeldedBatch->Shapes.Add(Shape);
						}
					}

					TArray<Chaos::FMassProperties> SubMassProperties;
					for (auto BodyShapesItr : BodyToShapes)
					{
						const FBodyInstance* OwningBI = BodyShapesItr.Key;
						const FWeldedBatch& WeldedBatch = BodyShapesItr.Value;

						// The component scale is already built into the geometry, but if the user has set up a CoM
						// modifier, it will need to be transformed by the component scale and the welded child's relative transform.
						FTransform MassModifierTransform = WeldedBatch.RelTM;
						MassModifierTransform.SetScale3D(MassModifierTransform.GetScale3D() * Scale3D);

						Chaos::FMassProperties BodyMassProperties = BodyUtils::ComputeMassProperties(OwningBI, WeldedBatch.Shapes, MassModifierTransform);
						SubMassProperties.Add(BodyMassProperties);
					}

					// Combine all the child inertias
					// NOTE: These leaves the inertia in diagonal form with a rotation of mass
					if (SubMassProperties.Num() > 0)
					{
						TotalMassProperties = Chaos::Combine(SubMassProperties);
					}
				}
				else
				{
					// If we have no shapes that affect mass we cannot compute the mass properties in a meaningful way.
					if (Shapes.Num() > 0)
					{
						// The component scale is already built into the geometry, but if the user has set up a CoM
						// modifier, it will need to be transformed by the component scale.
						const bool bInertiaScaleIncludeMass = false;
						FTransform MassModifierTransform(FQuat::Identity, FVector(0.f, 0.f, 0.f), Scale3D);
						TotalMassProperties = BodyUtils::ComputeMassProperties(this, Shapes, MassModifierTransform, bInertiaScaleIncludeMass);
					}
				}

				// Note: We expect the inertia to be diagonal at this point
				// Only set mass properties if inertia tensor is valid. TODO Remove this once we track down cause of empty tensors.
				// (This can happen if all shapes have bContributeToMass set to false which gives an empty Shapes array. There may be other ways).
				const float InertiaTensorTrace = (TotalMassProperties.InertiaTensor.M[0][0] + TotalMassProperties.InertiaTensor.M[1][1] + TotalMassProperties.InertiaTensor.M[2][2]) / 3;
				if (CHAOS_ENSURE(InertiaTensorTrace > UE_SMALL_NUMBER))
				{
					const FVector MassSpaceInertiaTensor(TotalMassProperties.InertiaTensor.M[0][0], TotalMassProperties.InertiaTensor.M[1][1], TotalMassProperties.InertiaTensor.M[2][2]);
					FPhysicsInterface::SetMassSpaceInertiaTensor_AssumesLocked(Actor, MassSpaceInertiaTensor);

					FPhysicsInterface::SetMass_AssumesLocked(Actor, TotalMassProperties.Mass);

					FTransform Com(TotalMassProperties.RotationOfMass, TotalMassProperties.CenterOfMass);
					FPhysicsInterface::SetComLocalPose_AssumesLocked(Actor, Com);
				}
			}
		});
	}

	//Let anyone who cares about mass properties know they've been updated
	if (BodyInstanceDelegates.IsValid())
	{
		BodyInstanceDelegates->OnRecalculatedMassProperties.Broadcast(this);
	}
}

void FBodyInstance::UpdateDebugRendering()
{
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	//After we update the mass properties, we should update any debug rendering
	if (UPrimitiveComponent* OwnerPrim = OwnerComponent.Get())
	{
		OwnerPrim->SendRenderDebugPhysics();
	}
#endif
}

void FBodyInstance::ApplySerializedState(Chaos::FSerializedDataBufferPtr&& InSerializedState, Chaos::ESerializedDataContext DataContext)
{
	const EPhysicsInterfaceScopedThreadContext LockContext = DataContext == Chaos::ESerializedDataContext::Internal ? EPhysicsInterfaceScopedThreadContext::Internal : EPhysicsInterfaceScopedThreadContext::External;
	FPhysicsCommand::ExecuteWrite(ActorHandle, [&InSerializedState, DataContext](const FPhysicsActorHandle& Actor)
	{
		FChaosEngineInterface::ApplySerializedStateToParticle(Actor, MoveTemp(InSerializedState), DataContext);
	}, LockContext, EPhysicsInterfaceScopedTransactionMode::Normal);

	// Collision geometry and shape data are not serialized for Dstm (see PushToPhysicsStateImp). 
	// So NonFrequentData (which is contains them) is marked as dirty to push those data into PT. 
	// This ensures ShapesArray is populated before shape properties are updated
	if (DataContext == Chaos::ESerializedDataContext::External)
	{
		MarkNonFrequentDirty();
	}
}

void FBodyInstance::MarkNonFrequentDirty()
{
	FPhysicsActorHandle PhysicsActor = GetPhysicsActor();
	if (PhysicsActor)
	{
		Chaos::FRigidBodyHandle_External& BodyExternal = PhysicsActor->GetGameThreadAPI();
		BodyExternal.MarkNonFrequentDataDirty();
	}
}

void FBodyInstance::SerializeState(Chaos::FSerializedDataBuffer& OutSerializedState, Chaos::ESerializedDataContext DataContext)
{
	const EPhysicsInterfaceScopedThreadContext LockContext = DataContext == Chaos::ESerializedDataContext::Internal ? EPhysicsInterfaceScopedThreadContext::Internal : EPhysicsInterfaceScopedThreadContext::External;
	FPhysicsCommand::ExecuteRead(ActorHandle, [&OutSerializedState, DataContext](const FPhysicsActorHandle& Actor)
	{
		FChaosEngineInterface::SerializeParticleStateToBuffer(Actor, OutSerializedState, DataContext);
	}, LockContext);
}

void FBodyInstance::UpdateDampingProperties()
{
	FPhysicsCommand::ExecuteWrite(GetPhysicsActor(), [&](FPhysicsActorHandle Actor)
	{
		if(FPhysicsInterface::IsDynamic(Actor))
	{
			FPhysicsInterface::SetLinearDamping_AssumesLocked(Actor, LinearDamping);
			FPhysicsInterface::SetAngularDamping_AssumesLocked(Actor, AngularDamping);
		}
	});
}

bool FBodyInstance::IsInstanceAwake() const
{
	bool bIsAwake = false;

	FPhysicsCommand::ExecuteRead(GetPhysicsActor(), [&](const FPhysicsActorHandle& Actor)
	{
		if(FPhysicsInterface::IsDynamic(Actor))
	{
			bIsAwake = !FPhysicsInterface::IsSleeping(Actor);
		}
	});

	return bIsAwake;
}

void FBodyInstance::WakeInstance()
{
	FPhysicsCommand::ExecuteWrite(GetPhysicsActor(), [&](FPhysicsActorHandle Actor)
	{
		if(FPhysicsInterface::IsDynamic(Actor) && FPhysicsInterface::IsInScene(Actor) && !FPhysicsInterface::IsKinematic_AssumesLocked(Actor))
		{
			FPhysicsInterface::WakeUp_AssumesLocked(Actor);
		}
	});
}

void FBodyInstance::PutInstanceToSleep()
{
	FPhysicsCommand::ExecuteWrite(GetPhysicsActor(), [&](const FPhysicsActorHandle& Actor)
	{
		if(FPhysicsInterface::IsDynamic(Actor) && FPhysicsInterface::IsInScene(Actor) && !FPhysicsInterface::IsKinematic_AssumesLocked(Actor))
		{
			FPhysicsInterface::PutToSleep_AssumesLocked(Actor);
		}
	});
}

float FBodyInstance::GetSleepThresholdMultiplier() const
{
	if (SleepFamily == ESleepFamily::Sensitive)
	{
		return SensitiveSleepThresholdMultiplier;
	}
	else if (SleepFamily == ESleepFamily::Custom)
	{
		return CustomSleepThresholdMultiplier;
	}

	return 1.f;
}

void FBodyInstance::SetLinearVelocity(const FVector& NewVel, bool bAddToCurrent, bool bAutoWake)
{

	FPhysicsCommand::ExecuteWrite(GetPhysicsActor(), [&](const FPhysicsActorHandle& Actor)
	{
		if(FPhysicsInterface::IsRigidBody(Actor))
	{
			FVector FinalVelocity = NewVel;

		if (bAddToCurrent)
		{
				FinalVelocity += FPhysicsInterface::GetLinearVelocity_AssumesLocked(Actor);
		}

			FPhysicsInterface::SetLinearVelocity_AssumesLocked(Actor, FinalVelocity);
		}
	});
}

void FBodyInstance::SetAngularVelocityInRadians(const FVector& NewAngVel, bool bAddToCurrent, bool bAutoWake)
{
	FPhysicsCommand::ExecuteWrite(GetPhysicsActor(), [&](const FPhysicsActorHandle& Actor)
	{
		if(FPhysicsInterface::IsRigidBody(Actor))
	{
			FVector FinalVelocity = NewAngVel;

		if (bAddToCurrent)
		{
				FinalVelocity += FPhysicsInterface::GetAngularVelocity_AssumesLocked(Actor);
		}

			FPhysicsInterface::SetAngularVelocity_AssumesLocked(Actor, FinalVelocity);
		}
	});
}

float FBodyInstance::GetMaxAngularVelocityInRadians() const
{
	return bOverrideMaxAngularVelocity ? FMath::DegreesToRadians(MaxAngularVelocity) : FMath::DegreesToRadians(UPhysicsSettings::Get()->MaxAngularVelocity);
}

void FBodyInstance::SetMaxAngularVelocityInRadians(float NewMaxAngVel, bool bAddToCurrent, bool bUpdateOverrideMaxAngularVelocity)
{
	float NewMaxInDegrees = FMath::RadiansToDegrees(NewMaxAngVel);

	FPhysicsCommand::ExecuteWrite(GetPhysicsActor(), [&](const FPhysicsActorHandle& Actor)
	{
		if (bAddToCurrent)
		{
			float OldValue = FPhysicsInterface::GetMaxAngularVelocity_AssumesLocked(Actor);
			NewMaxAngVel += OldValue;
			float OldValueInDegrees = FMath::RadiansToDegrees(OldValue);
			NewMaxInDegrees += OldValueInDegrees;
		}

		FPhysicsInterface::SetMaxAngularVelocity_AssumesLocked(Actor, NewMaxAngVel);
	});

	MaxAngularVelocity = NewMaxInDegrees;

	if(bUpdateOverrideMaxAngularVelocity)
	{
		bOverrideMaxAngularVelocity = true;
	}
}

void FBodyInstance::SetOverrideMaxDepenetrationVelocity(bool bInEnabled)
{
	bOverrideMaxDepenetrationVelocity = bInEnabled;

	UpdateMaxDepenetrationVelocity();
}

void FBodyInstance::SetMaxDepenetrationVelocity(float MaxVelocity)
{
	bOverrideMaxDepenetrationVelocity = true;
	MaxDepenetrationVelocity = MaxVelocity;

	UpdateMaxDepenetrationVelocity();
}

void FBodyInstance::UpdateMaxDepenetrationVelocity()
{
	// Negative values mean do not use max depenetration velocity (equivalent to large number)
	const float UsedMaxDepenetrationVelocity = (bOverrideMaxDepenetrationVelocity) ? MaxDepenetrationVelocity : -1.0f;

	// NOTE: FBodyInstance::MaxDepentrationVelocity now means initial depenetration velocity, 
	// and not the general solver depentration velocity limit (which will probably be removed)
	FPhysicsCommand::ExecuteWrite(GetPhysicsActor(), [this, UsedMaxDepenetrationVelocity](const FPhysicsActorHandle& Actor)
	{
		FPhysicsInterface::SetMaxDepenetrationVelocity_AssumesLocked(Actor, UsedMaxDepenetrationVelocity);
	});
}

void FBodyInstance::AddCustomPhysics(FCalculateCustomPhysics& CalculateCustomPhysics)
{
	FPhysicsCommand::ExecuteWrite(GetPhysicsActor(), [&](const FPhysicsActorHandle& Actor)
	{
		if(!IsRigidBodyKinematic_AssumesLocked(Actor))
		{
			if(FPhysScene* PhysScene = GetPhysicsScene())
			{
				PhysScene->AddCustomPhysics_AssumesLocked(this, CalculateCustomPhysics);
			}
		}
	});
}

void FBodyInstance::ApplyAsyncPhysicsCommand(FAsyncPhysicsTimestamp TimeStamp, const bool bIsInternal, APlayerController* PlayerController, const TFunction<void()>& Command)
{
	if (bIsInternal && TimeStamp.IsValid())
	{
		APlayerController* LocalController = PlayerController ? PlayerController : 
			OwnerComponent->GetWorld() ? OwnerComponent->GetWorld()->GetFirstPlayerController() : nullptr;
		if (LocalController)
		{
			TimeStamp.LocalFrame = TimeStamp.ServerFrame - LocalController->GetNetworkPhysicsTickOffset();
			LocalController->ExecuteAsyncPhysicsCommand(TimeStamp, OwnerComponent.Get(), Command);
		}
	}
	else
	{
		FPhysicsCommand::ExecuteWrite(GetPhysicsActor(), [&](const FPhysicsActorHandle& Actor)
			{
				Command();
			});
	}
}

static bool IsBodyDynamic(const FPhysicsActorHandle& ActorHandle, const bool bIsInternal)
{
	const Chaos::EObjectStateType ObjectState = bIsInternal ? ActorHandle->GetPhysicsThreadAPI()->ObjectState() :
		ActorHandle->GetGameThreadAPI().ObjectState();

	return (ObjectState == Chaos::EObjectStateType::Dynamic) || (ObjectState == Chaos::EObjectStateType::Sleeping);
}

void FBodyInstance::AddForce(const FVector& Force, bool bAllowSubstepping, bool bAccelChange, const FAsyncPhysicsTimestamp TimeStamp, APlayerController* PlayerController)
{
	const bool bIsInternal = TimeStamp.IsValid();
	ApplyAsyncPhysicsCommand(TimeStamp, bIsInternal, PlayerController, [&, Force, bAllowSubstepping, bAccelChange, bIsInternal]()
	{
		if (FPhysicsInterface::IsInScene(GetPhysicsActor()) && IsBodyDynamic(GetPhysicsActor(), bIsInternal))
		{
			FPhysicsInterface::AddForce_AssumesLocked(GetPhysicsActor(), Force, bAllowSubstepping, bAccelChange, bIsInternal);
		}
	});
}

void FBodyInstance::AddForceAtPosition(const FVector& Force, const FVector& Position, bool bAllowSubstepping, bool bIsLocalForce, const FAsyncPhysicsTimestamp TimeStamp, APlayerController* PlayerController)
{
	const bool bIsInternal = TimeStamp.IsValid();
	ApplyAsyncPhysicsCommand(TimeStamp, bIsInternal, PlayerController, [&, Force, Position, bAllowSubstepping, bIsLocalForce, bIsInternal]()
	{
		if (FPhysicsInterface::IsInScene(GetPhysicsActor()) && IsBodyDynamic(GetPhysicsActor(), bIsInternal))
		{
			FPhysicsInterface::AddForceAtPosition_AssumesLocked(GetPhysicsActor(), Force, Position, bAllowSubstepping, bIsLocalForce, bIsInternal);
		}
	});
}

void FBodyInstance::ClearForces(bool bAllowSubstepping)
{
	FPhysicsCommand::ExecuteWrite(GetPhysicsActor(), [&](const FPhysicsActorHandle& Actor)
	{
		if(!IsRigidBodyKinematic_AssumesLocked(Actor))
		{
			if(FPhysScene* PhysScene = GetPhysicsScene())
			{
				PhysScene->ClearForces_AssumesLocked(this, bAllowSubstepping);
			}
		}
	});
}

void FBodyInstance::SetOneWayInteraction(bool bInOneWayInteraction)
{
	if (bOneWayInteraction != bInOneWayInteraction)
	{
		bOneWayInteraction = bInOneWayInteraction;

		UpdateOneWayInteraction();
	}
}

void FBodyInstance::UpdateOneWayInteraction()
{
	const bool bCurrentOneWayInteraction = bOneWayInteraction;

	FPhysicsCommand::ExecuteWrite(GetPhysicsActor(), [bCurrentOneWayInteraction](const FPhysicsActorHandle& Actor)
	{
		if (FPhysicsInterface::IsRigidBody(Actor) && !IsRigidBodyKinematic_AssumesLocked(Actor))
		{
			FPhysicsInterface::SetOneWayInteraction_AssumesLocked(Actor, bCurrentOneWayInteraction);
		}
	});
}

void FBodyInstance::AddTorqueInRadians(const FVector& Torque, bool bAllowSubstepping, bool bAccelChange, const FAsyncPhysicsTimestamp TimeStamp, APlayerController* PlayerController)
{
	const bool bIsInternal = TimeStamp.IsValid();
	ApplyAsyncPhysicsCommand(TimeStamp, bIsInternal, PlayerController, [&, Torque, bAllowSubstepping, bAccelChange, bIsInternal]()
	{
		if (FPhysicsInterface::IsInScene(GetPhysicsActor()) && IsBodyDynamic(GetPhysicsActor(), bIsInternal))
		{
			FPhysicsInterface::AddTorque_AssumesLocked(GetPhysicsActor(), Torque, bAllowSubstepping, bAccelChange, bIsInternal);
		}
	});
}

void FBodyInstance::ClearTorques(bool bAllowSubstepping)
{
	FPhysicsCommand::ExecuteWrite(GetPhysicsActor(), [&](const FPhysicsActorHandle& Actor)
	{
		if(!IsRigidBodyKinematic_AssumesLocked(Actor))
		{
			if(FPhysScene* PhysScene = GetPhysicsScene())
			{
				PhysScene->ClearTorques_AssumesLocked(this, bAllowSubstepping);
			}
		}
	});
}

void FBodyInstance::AddAngularImpulseInRadians(const FVector& AngularImpulse, bool bVelChange, const FAsyncPhysicsTimestamp TimeStamp, APlayerController* PlayerController)
{
	const bool bIsInternal = TimeStamp.IsValid();
	ApplyAsyncPhysicsCommand(TimeStamp, bIsInternal, PlayerController, [&, AngularImpulse, bVelChange, bIsInternal]()
	{
		if (FPhysicsInterface::IsInScene(GetPhysicsActor()) && IsBodyDynamic(GetPhysicsActor(), bIsInternal))
		{
			if (bVelChange)
			{
				FPhysicsInterface::AddAngularVelocityInRadians_AssumesLocked(GetPhysicsActor(), AngularImpulse, bIsInternal);
			}
			else
			{
				FPhysicsInterface::AddAngularImpulseInRadians_AssumesLocked(GetPhysicsActor(), AngularImpulse, bIsInternal);
			}
		}
	});
}

void FBodyInstance::AddImpulse(const FVector& Impulse, bool bVelChange, const FAsyncPhysicsTimestamp TimeStamp, APlayerController* PlayerController)
{
	const bool bIsInternal = TimeStamp.IsValid();
	ApplyAsyncPhysicsCommand(TimeStamp, bIsInternal, PlayerController, [&, Impulse, bVelChange, bIsInternal]()
	{		
		if (FPhysicsInterface::IsInScene(GetPhysicsActor()) && IsBodyDynamic(GetPhysicsActor(), bIsInternal))
		{
			if (bVelChange)
			{
				FPhysicsInterface::AddVelocity_AssumesLocked(GetPhysicsActor(), Impulse, bIsInternal);
			}
			else
			{
				FPhysicsInterface::AddImpulse_AssumesLocked(GetPhysicsActor(), Impulse, bIsInternal);
			}
		}
	});
}

void FBodyInstance::AddImpulseAtPosition(const FVector& Impulse, const FVector& Position, const FAsyncPhysicsTimestamp TimeStamp, APlayerController* PlayerController)
{
	const bool bIsInternal = TimeStamp.IsValid();
	ApplyAsyncPhysicsCommand(TimeStamp, bIsInternal, PlayerController, [&, Impulse, Position, bIsInternal]()
	{
		if (FPhysicsInterface::IsInScene(GetPhysicsActor()) && IsBodyDynamic(GetPhysicsActor(), bIsInternal))
		{
			FPhysicsInterface::AddImpulseAtLocation_AssumesLocked(GetPhysicsActor(), Impulse, Position, bIsInternal);
		}
	});
}

void FBodyInstance::AddVelocityChangeImpulseAtLocation(const FVector& Impulse, const FVector& Position, const FAsyncPhysicsTimestamp TimeStamp, APlayerController* PlayerController)
{
	const bool bIsInternal = TimeStamp.IsValid();
	ApplyAsyncPhysicsCommand(TimeStamp, bIsInternal, PlayerController, [&, Impulse, Position, bIsInternal]()
	{
		if (FPhysicsInterface::IsInScene(GetPhysicsActor()) && IsBodyDynamic(GetPhysicsActor(), bIsInternal))
		{
			FPhysicsInterface::AddVelocityChangeImpulseAtLocation_AssumesLocked(GetPhysicsActor(), Impulse, Position, bIsInternal);
		}
	});
}

void FBodyInstance::SetInstanceNotifyRBCollision(bool bNewNotifyCollision)
{

	if (bNewNotifyCollision == bNotifyRigidBodyCollision)
	{
		// don't update if we've already set it
		return;
	}
	
	// make sure to register the component for collision events 
	if (UPrimitiveComponent* PrimComp = OwnerComponent.Get())
	{
		if (UWorld* World = PrimComp->GetWorld())
		{
			if (FPhysScene_Chaos* PhysScene = World->GetPhysicsScene())
			{
				if (bNewNotifyCollision)
				{
					// add to the list
					PhysScene->RegisterForCollisionEvents(PrimComp);
				}
				else
				{
					PhysScene->UnRegisterForCollisionEvents(PrimComp);
				}
			}
		}
	}

	bNotifyRigidBodyCollision = bNewNotifyCollision;
	UpdatePhysicsFilterData();
}

void FBodyInstance::SetEnableGravity(bool bInGravityEnabled)
{
	if (bEnableGravity != bInGravityEnabled)
	{
		bEnableGravity = bInGravityEnabled;

		{
			FPhysicsCommand::ExecuteWrite(GetPhysicsActor(), [&](const FPhysicsActorHandle& Actor)
			{
				if(FPhysicsInterface::IsRigidBody(Actor))
				{
					FPhysicsInterface::SetGravityEnabled_AssumesLocked(Actor, bEnableGravity);
				}
		});
		}

		if (bEnableGravity)
		{
			WakeInstance();
		}
	}
}

void FBodyInstance::SetGravityGroupIndex(int32 NewGravityGroupIndex)
{
	if (NewGravityGroupIndex >= 0)
	{
		GravityGroupIndex = NewGravityGroupIndex;

		{
			FPhysicsCommand::ExecuteWrite(GetPhysicsActor(), [&](const FPhysicsActorHandle& Actor)
				{
					if (FPhysicsInterface::IsRigidBody(Actor))
					{
						FPhysicsInterface::SetGravityGroupIndex_AssumesLocked(Actor, GravityGroupIndex);
					}
				});
		}
	}
}

void FBodyInstance::SetUpdateKinematicFromSimulation(bool bInUpdateKinematicFromSimulation)
{
	if (bUpdateKinematicFromSimulation != bInUpdateKinematicFromSimulation)
	{
		bUpdateKinematicFromSimulation = bInUpdateKinematicFromSimulation;

		{
			FPhysicsCommand::ExecuteWrite(GetPhysicsActor(), [&](const FPhysicsActorHandle& Actor)
				{
					if (FPhysicsInterface::IsRigidBody(Actor))
					{
						FPhysicsInterface::SetUpdateKinematicFromSimulation_AssumesLocked(Actor, bUpdateKinematicFromSimulation);
					}
				});
		}
	}
}

void FBodyInstance::SetGyroscopicTorqueEnabled(bool bInGyroscopicTorqueEnabled)
{
	if (bGyroscopicTorqueEnabled != bInGyroscopicTorqueEnabled)
	{
		bGyroscopicTorqueEnabled = bInGyroscopicTorqueEnabled;

		{
			FPhysicsCommand::ExecuteWrite(GetPhysicsActor(), [&](const FPhysicsActorHandle& Actor)
				{
					if (FPhysicsInterface::IsRigidBody(Actor))
					{
						FPhysicsInterface::SetGyroscopicTorqueEnabled_AssumesLocked(Actor, bGyroscopicTorqueEnabled);
					}
				});
		}
	}
}


void FBodyInstance::SetContactModification(bool bNewContactModification)
{
	if (bNewContactModification != bContactModification)
	{
		bContactModification = bNewContactModification;
		UpdatePhysicsFilterData();
	}
}

void FBodyInstance::SetSmoothEdgeCollisionsEnabled(bool bNewSmoothEdgeCollisions)
{
	if (bNewSmoothEdgeCollisions != bSmoothEdgeCollisions)
	{
		bSmoothEdgeCollisions = bNewSmoothEdgeCollisions;

		FPhysicsCommand::ExecuteWrite(GetPhysicsActor(),
			[bNewSmoothEdgeCollisions](const FPhysicsActorHandle& Actor)
			{
				if (FPhysicsInterface::IsRigidBody(Actor))
				{
					FPhysicsInterface::SetSmoothEdgeCollisionsEnabled_AssumesLocked(Actor, bNewSmoothEdgeCollisions);
				}
			});
	}
}

void FBodyInstance::SetUseCCD(bool bInUseCCD)
{
	if (bUseCCD != bInUseCCD)
	{
		bUseCCD = bInUseCCD;
		// Need to set body flag
		FPhysicsCommand::ExecuteWrite(GetPhysicsActor(), [&](const FPhysicsActorHandle& Actor)
		{
			if (FPhysicsInterface::IsValid(Actor) && FPhysicsInterface::IsRigidBody(Actor))
			{
				FPhysicsInterface::SetCcdEnabled_AssumesLocked(Actor, bUseCCD);
			}
		});
		// And update collision filter data
		UpdatePhysicsFilterData();
	}
}

void FBodyInstance::SetUseMACD(bool bInUseMACD)
{
	if (bUseMACD != bInUseMACD)
	{
		bUseMACD = bInUseMACD;

		FPhysicsCommand::ExecuteWrite(GetPhysicsActor(), [this, bInUseMACD](const FPhysicsActorHandle& Actor)
			{
				if (FPhysicsInterface::IsValid(Actor) && FPhysicsInterface::IsRigidBody(Actor))
				{
					FPhysicsInterface::SetMACDEnabled_AssumesLocked(Actor, bInUseMACD);
				}
			});
	}
}

void FBodyInstance::SetAllowPartialIslandSleep(bool bInAllowPartialIslandSleep)
{
	if (bAllowPartialIslandSleep != bInAllowPartialIslandSleep)
	{
		bAllowPartialIslandSleep = bInAllowPartialIslandSleep;

		FPhysicsCommand::ExecuteWrite(GetPhysicsActor(), [this, bInAllowPartialIslandSleep](const FPhysicsActorHandle& Actor)
			{
				if (FPhysicsInterface::IsValid(Actor) && FPhysicsInterface::IsRigidBody(Actor))
				{
					FPhysicsInterface::SetPartialIslandSleepAllowed_AssumesLocked(Actor, bInAllowPartialIslandSleep);
				}
			});
	}
}

void FBodyInstance::SetOverrideIterationCounts(bool bOverride)
{
	if (bOverride != bOverrideIterationCounts)
	{
		bOverrideIterationCounts = bOverride;

		int32 PosCount = GetPositionSolverIterationCount();
		int32 VelCount = GetVelocitySolverIterationCount();
		int32 ProjCount = GetProjectionSolverIterationCount();

		FPhysicsCommand::ExecuteWrite(GetPhysicsActor(), [this, PosCount, VelCount, ProjCount](const FPhysicsActorHandle& Actor)
			{
				if (FPhysicsInterface::IsValid(Actor) && FPhysicsInterface::IsRigidBody(Actor))
				{
					FPhysicsInterface::SetPositionSolverIterationCount_AssumesLocked(Actor, PosCount);
					FPhysicsInterface::SetVelocitySolverIterationCount_AssumesLocked(Actor, VelCount);
					FPhysicsInterface::SetProjectionSolverIterationCount_AssumesLocked(Actor, ProjCount);
				}
			});
	}
}

void FBodyInstance::SetPositionSolverIterationCount(uint8 PositionSolverIterationCountIn)
{
	if (PositionSolverIterationCount != PositionSolverIterationCountIn)
	{
		PositionSolverIterationCount = PositionSolverIterationCountIn;

		int32 PosCount = GetPositionSolverIterationCount();

		FPhysicsCommand::ExecuteWrite(GetPhysicsActor(), [this, PosCount](const FPhysicsActorHandle& Actor)
			{
				if (FPhysicsInterface::IsValid(Actor) && FPhysicsInterface::IsRigidBody(Actor))
				{
					FPhysicsInterface::SetPositionSolverIterationCount_AssumesLocked(Actor, PosCount);
				}
			});
	}
}

void FBodyInstance::SetVelocitySolverIterationCount(uint8 VelocitySolverIterationCountIn)
{
	if (VelocitySolverIterationCount != VelocitySolverIterationCountIn)
	{
		VelocitySolverIterationCount = VelocitySolverIterationCountIn;

		int32 VelCount = GetVelocitySolverIterationCount();

		FPhysicsCommand::ExecuteWrite(GetPhysicsActor(), [this, VelCount](const FPhysicsActorHandle& Actor)
			{
				if (FPhysicsInterface::IsValid(Actor) && FPhysicsInterface::IsRigidBody(Actor))
				{
					FPhysicsInterface::SetVelocitySolverIterationCount_AssumesLocked(Actor, VelCount);
				}
			});
	}
}

void FBodyInstance::SetProjectionSolverIterationCount(uint8 ProjectionSolverIterationCountIn)
{
	if (ProjectionSolverIterationCount != ProjectionSolverIterationCountIn)
	{
		ProjectionSolverIterationCount = ProjectionSolverIterationCountIn;

		int32 ProjCount = GetProjectionSolverIterationCount();

		FPhysicsCommand::ExecuteWrite(GetPhysicsActor(), [this, ProjCount](const FPhysicsActorHandle& Actor)
			{
				if (FPhysicsInterface::IsValid(Actor) && FPhysicsInterface::IsRigidBody(Actor))
				{
					FPhysicsInterface::SetProjectionSolverIterationCount_AssumesLocked(Actor, ProjCount);
				}
			});
	}
}

void FBodyInstance::SetPhysicsDisabled(bool bSetDisabled)
{
	FPhysicsCommand::ExecuteWrite(GetPhysicsActor(), [&](const FPhysicsActorHandle& Actor)
	{
		if (FPhysicsInterface::IsValid(Actor) && FPhysicsInterface::IsRigidBody(Actor))
		{
			FPhysicsInterface::SetDisabled(Actor, bSetDisabled);
		}
	});
}

bool FBodyInstance::IsPhysicsDisabled() const
{
	bool bIsDisabled = false;
	FPhysicsCommand::ExecuteRead(GetPhysicsActor(), [&](const FPhysicsActorHandle& Actor)
	{
		bIsDisabled = FPhysicsInterface::IsDisabled(Actor);
	});
	return bIsDisabled;
}

EPhysicsReplicationMode FBodyInstance::GetPhysicsReplicationMode() const
{
	UPrimitiveComponent* OwnerComponentInst = OwnerComponent.Get();
	AActor* OwningActor = OwnerComponentInst ? OwnerComponentInst->GetOwner() : nullptr;
	if (OwningActor)
	{
		return OwningActor->GetPhysicsReplicationMode();
	}

	return EPhysicsReplicationMode::Default;
}

void FBodyInstance::AddRadialImpulseToBody(const FVector& Origin, float Radius, float Strength, uint8 Falloff, bool bVelChange)
{
	FPhysicsCommand::ExecuteWrite(GetPhysicsActor(), [&](const FPhysicsActorHandle& Actor)
	{
		if(FPhysicsInterface::IsRigidBody(Actor) && FPhysicsInterface::IsInScene(Actor) && !IsRigidBodyKinematic_AssumesLocked(Actor))
		{
			FPhysicsInterface::AddRadialImpulse_AssumesLocked(Actor, Origin, Radius, Strength, (ERadialImpulseFalloff)Falloff, bVelChange);
		}
	});
}

void FBodyInstance::AddRadialForceToBody(const FVector& Origin, float Radius, float Strength, uint8 Falloff, bool bAccelChange, bool bAllowSubstepping)
{
	FPhysicsCommand::ExecuteWrite(GetPhysicsActor(), [&](const FPhysicsActorHandle& Actor)
	{
		if(FPhysicsInterface::IsRigidBody(Actor) && FPhysicsInterface::IsInScene(Actor) && !IsRigidBodyKinematic_AssumesLocked(Actor))
		{
			if(FPhysScene* PhysScene = GetPhysicsScene())
			{
				PhysScene->AddRadialForceToBody_AssumesLocked(this, Origin, Radius, Strength, Falloff, bAccelChange, bAllowSubstepping);
			}
		}
	});
}

FString FBodyInstance::GetBodyDebugName() const
{
	FString DebugName;

	UPrimitiveComponent* OwnerComponentInst = OwnerComponent.Get();
	if (OwnerComponentInst != NULL)
	{
		DebugName = OwnerComponentInst->GetPathName();
		if (const UObject* StatObject = OwnerComponentInst->AdditionalStatObject())
		{
			DebugName += TEXT(" ");
			StatObject->AppendName(DebugName);
		}
	}

	if ((BodySetup != NULL) && (BodySetup->BoneName != NAME_None))
	{
		DebugName += FString(" Bone: ") + BodySetup->BoneName.ToString();
	}

	return DebugName;
}


/////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
// COLLISION

bool FBodyInstance::LineTrace(struct FHitResult& OutHit, const FVector& Start, const FVector& End, bool bTraceComplex, bool bReturnPhysicalMaterial) const
{
	SCOPE_CYCLE_COUNTER(STAT_Collision_SceneQueryTotal);
	SCOPE_CYCLE_COUNTER(STAT_Collision_FBodyInstance_LineTrace);

	return FPhysicsInterface::LineTrace_Geom(OutHit, this, Start, End, bTraceComplex, bReturnPhysicalMaterial);
}

bool FBodyInstance::Sweep(struct FHitResult& OutHit, const FVector& Start, const FVector& End, const FQuat& ShapeWorldRotation, const FCollisionShape& CollisionShape, bool bTraceComplex) const
{
	return FPhysicsInterface::Sweep_Geom(OutHit, this, Start, End, ShapeWorldRotation, CollisionShape, bTraceComplex);
		}

bool FBodyInstance::GetSquaredDistanceToBody(const FVector& Point, float& OutDistanceSquared, FVector& OutPointOnBody) const
{
	return FPhysicsInterface::GetSquaredDistanceToBody(this, Point, OutDistanceSquared, &OutPointOnBody);
}

float FBodyInstance::GetDistanceToBody(const FVector& Point, FVector& OutPointOnBody) const
{
	float DistanceSqr = -1.f;
	return (GetSquaredDistanceToBody(Point, DistanceSqr, OutPointOnBody) ? FMath::Sqrt(DistanceSqr) : -1.f);
}

template <typename AllocatorType>
bool FBodyInstance::OverlapTestForBodiesImpl(const FVector& Pos, const FQuat& Rot, const TArray<FBodyInstance*, AllocatorType>& Bodies, bool bTraceComplex) const
{
	bool bHaveOverlap = false;

	FPhysicsCommand::ExecuteRead(GetPhysicsActor(), [&](const FPhysicsActorHandle& Actor)
	{
		// calculate the test global pose of the rigid body
		FTransform PTestGlobalPose = FTransform(Rot, Pos);

		// Get all the shapes from the actor
		FInlineShapeArray TargetShapes;

		const int32 NumTargetShapes = FillInlineShapeArray_AssumesLocked(TargetShapes, Actor);

		for(const FPhysicsShapeHandle& Shape : TargetShapes)
		{
			if (!Shape.GetGeometry().IsConvex())
			{
				continue;	//we skip complex shapes - should this respect ComplexAsSimple?
			}

			for (const FBodyInstance* BodyInstance : Bodies)
			{
				bHaveOverlap = FPhysicsInterface::Overlap_Geom(BodyInstance, FPhysicsInterface::GetGeometryCollection(Shape), PTestGlobalPose, /*bOutMTD=*/nullptr, bTraceComplex);

				if (bHaveOverlap)
				{
					return;
				}
			}
		}
	});
	return bHaveOverlap;
}

// Explicit template instantiation for the above.
template ENGINE_API bool FBodyInstance::OverlapTestForBodiesImpl(const FVector& Pos, const FQuat& Rot, const TArray<FBodyInstance*>& Bodies, bool bTraceComplex) const;
template ENGINE_API bool FBodyInstance::OverlapTestForBodiesImpl(const FVector& Pos, const FQuat& Rot, const TArray<FBodyInstance*, TInlineAllocator<1>>& Bodies, bool bTraceComplex) const;


bool FBodyInstance::OverlapTest(const FVector& Position, const FQuat& Rotation, const struct FCollisionShape& CollisionShape, FMTDResult* OutMTD, bool bTraceComplex) const
{
	SCOPE_CYCLE_COUNTER(STAT_Collision_SceneQueryTotal);
	SCOPE_CYCLE_COUNTER(STAT_Collision_FBodyInstance_OverlapTest);

	bool bHasOverlap = false;

	FPhysicsCommand::ExecuteRead(GetPhysicsActor(), [&](const FPhysicsActorHandle& Actor)
	{
		FTransform GeomTransform(Rotation, Position);

		bHasOverlap = FPhysicsInterface::Overlap_Geom(this, CollisionShape, Rotation, GeomTransform, OutMTD, bTraceComplex);
	});

	return bHasOverlap;
}

bool FBodyInstance::OverlapTest_AssumesLocked(const FVector& Position, const FQuat& Rotation, const struct FCollisionShape& CollisionShape, FMTDResult* OutMTD /*= nullptr*/, bool bTraceComplex) const
{
	SCOPE_CYCLE_COUNTER(STAT_Collision_SceneQueryTotal);
	SCOPE_CYCLE_COUNTER(STAT_Collision_FBodyInstance_OverlapTest);

	FTransform GeomTransform(Rotation, Position);
	bool bHasOverlap = FPhysicsInterface::Overlap_Geom(this, CollisionShape, Rotation, GeomTransform, OutMTD, bTraceComplex);
	return bHasOverlap;
}

FTransform RootSpaceToWeldedSpace(const FBodyInstance* BI, const FTransform& RootTM)
{
	if (BI->WeldParent)
	{
		UPrimitiveComponent* Parent = BI->WeldParent->OwnerComponent.Get();
		UPrimitiveComponent* Child = BI->OwnerComponent.Get();
		if (Parent && Child)
		{
			FTransform ParentT = Parent->GetComponentTransform();
			FTransform ChildT = Child->GetComponentTransform();
			FTransform RelativeT = ParentT.GetRelativeTransform(ChildT);
			FTransform ScaledRoot = RootTM;
			ScaledRoot.SetScale3D(BI->Scale3D);
			FTransform FinalTM = RelativeT * ScaledRoot;
			FinalTM.SetScale3D(FVector::OneVector);

			return FinalTM;
		}
	}

	return RootTM;
}

bool FBodyInstance::OverlapMulti(TArray<struct FOverlapResult>& InOutOverlaps, const class UWorld* World, const FTransform* pWorldToComponent, const FVector& Pos, const FQuat& Quat, ECollisionChannel TestChannel, const struct FComponentQueryParams& Params, const struct FCollisionResponseParams& ResponseParams, const FCollisionObjectQueryParams& ObjectQueryParams) const
{
	SCOPE_CYCLE_COUNTER(STAT_Collision_SceneQueryTotal);
	SCOPE_CYCLE_COUNTER(STAT_Collision_FBodyInstance_OverlapMulti);

	if ( !IsValidBodyInstance()  && (!WeldParent || !WeldParent->IsValidBodyInstance()))
	{
		UE_LOGF(LogCollision, Log, "FBodyInstance::OverlapMulti : (%ls) No physics data", *GetBodyDebugName());
		return false;
	}

	bool bHaveBlockingHit = false;

	// Determine how to convert the local space of this body instance to the test space
	const FTransform ComponentSpaceToTestSpace(Quat, Pos);

	FTransform BodyInstanceSpaceToTestSpace(NoInit);
	if (pWorldToComponent)
	{
		const FTransform RootTM = WeldParent ? WeldParent->GetUnrealWorldTransform() : GetUnrealWorldTransform();
		const FTransform LocalOffset = (*pWorldToComponent) * RootTM;
		BodyInstanceSpaceToTestSpace = ComponentSpaceToTestSpace * LocalOffset;
	}
	else
	{
		BodyInstanceSpaceToTestSpace = ComponentSpaceToTestSpace;
	}

	//We want to test using global position. However, the global position of the body will be in terms of the root body which we are welded to. So we must undo the relative transform so that our shapes are centered
	//Global = Parent * Relative => Global * RelativeInverse = Parent
	if (WeldParent)
	{
		BodyInstanceSpaceToTestSpace = RootSpaceToWeldedSpace(this, BodyInstanceSpaceToTestSpace);
	}

	const FBodyInstance* TargetInstance = WeldParent ? WeldParent : this;

	FPhysicsCommand::ExecuteRead(TargetInstance->GetPhysicsActor(), [&](const FPhysicsActorHandle& Actor)
	{
		if(FPhysicsInterface::IsValid(Actor))
		{
		// Get all the shapes from the actor
			FInlineShapeArray PShapes;
			const int32 NumShapes = FillInlineShapeArray_AssumesLocked(PShapes, Actor);

			// Iterate over each shape
			TArray<struct FOverlapResult> TempOverlaps;
			for (int32 ShapeIdx = 0; ShapeIdx < NumShapes; ShapeIdx++)
			{
				// Skip this shape if it's CollisionEnabled setting was masked out
				if (Params.ShapeCollisionMask && !(Params.ShapeCollisionMask & GetShapeCollisionEnabled(ShapeIdx)))
				{
					continue;
				}

				FPhysicsShapeHandle& ShapeRef = PShapes[ShapeIdx];

				FPhysicsGeometryCollection GeomCollection = FPhysicsInterface::GetGeometryCollection(ShapeRef);

				if(IsShapeBoundToBody(ShapeRef) == false)
				{
					continue;
				}

				if (!ShapeRef.GetGeometry().IsConvex())
				{
					continue;	//we skip complex shapes - should this respect ComplexAsSimple?
				}

				TempOverlaps.Reset();
				if(FPhysicsInterface::GeomOverlapMulti(World, GeomCollection, BodyInstanceSpaceToTestSpace.GetTranslation(), BodyInstanceSpaceToTestSpace.GetRotation(), TempOverlaps, TestChannel, Params, ResponseParams, ObjectQueryParams))
				{
					bHaveBlockingHit = true;
				}

				// We need to make sure the results get the InstanceBodyIndex
				for (FOverlapResult& Result : TempOverlaps)
				{
					Result.MyItemIndex = InstanceBodyIndex;
				}

				InOutOverlaps.Append(TempOverlaps);
			}
			}
		});

	return bHaveBlockingHit;
}

bool FBodyInstance::IsValidCollisionProfileName(FName InCollisionProfileName)
{
	return (InCollisionProfileName != NAME_None) && (InCollisionProfileName != UCollisionProfile::CustomCollisionProfileName);
}

void FBodyInstance::LoadProfileData(bool bVerifyProfile)
{
	const FName UseCollisionProfileName = GetCollisionProfileName();
	if ( bVerifyProfile )
	{
		// if collision profile name exists, 
		// check with current settings
		// if same, then keep the profile name
		// if not same, that means it has been modified from default
		// leave it as it is, and clear profile name
		if ( IsValidCollisionProfileName(UseCollisionProfileName) )
		{
			FCollisionResponseTemplate Template;
			if ( UCollisionProfile::Get()->GetProfileTemplate(UseCollisionProfileName, Template) ) 
			{
				// this function is only used for old code that did require verification of using profile or not
				// so that means it will have valid ResponsetoChannels value, so this is okay to access. 
				if (Template.IsEqual(CollisionEnabled, ObjectType, CollisionResponses.GetResponseContainer()) == false)
				{
					InvalidateCollisionProfileName(); 
				}
			}
			else
			{
				UE_LOGF(LogPhysics, Warning, "COLLISION PROFILE [%ls] is not found", *UseCollisionProfileName.ToString());
				// if not nothing to do
				InvalidateCollisionProfileName(); 
			}
		}
	}
	else
	{
		if ( IsValidCollisionProfileName(UseCollisionProfileName) )
		{
			if ( UCollisionProfile::Get()->ReadConfig(UseCollisionProfileName, *this) == false)
			{
				// clear the name
				InvalidateCollisionProfileName();
			}
		}

		// no profile, so it just needs to update container from array data
		if ( DoesUseCollisionProfile() == false )
		{
			// if external profile copy the data over
			if (ExternalCollisionProfileBodySetup.IsValid(true))
			{
				UBodySetup* BodySetupInstance = ExternalCollisionProfileBodySetup.Get(true);
				const FBodyInstance& ExternalBodyInstance = BodySetupInstance->DefaultInstance;
				CollisionProfileName = ExternalBodyInstance.CollisionProfileName;
				ObjectType = ExternalBodyInstance.ObjectType;
				CollisionEnabled = ExternalBodyInstance.CollisionEnabled;
				CollisionResponses.SetCollisionResponseContainer(ExternalBodyInstance.CollisionResponses.ResponseToChannels);
			}
			else
			{
				CollisionResponses.UpdateResponseContainerFromArray();
			}
		}
	}
}

void FBodyInstance::GetBodyInstanceResourceSizeEx(FResourceSizeEx& CumulativeResourceSize) const
{
	CumulativeResourceSize.AddDedicatedSystemMemoryBytes(FPhysicsInterface::GetResourceSizeEx(GetPhysicsActor()));
}

void FBodyInstance::FixupData(class UObject* Loader)
{
	check (Loader);

	FPackageFileVersion const UEVersion = Loader->GetLinkerUEVersion();

#if WITH_EDITOR
	if (UEVersion < VER_UE4_ADD_CUSTOMPROFILENAME_CHANGE)
	{
		if (CollisionProfileName == NAME_None)
		{
			CollisionProfileName = UCollisionProfile::CustomCollisionProfileName;
		}
	}

	if (UEVersion < VER_UE4_SAVE_COLLISIONRESPONSE_PER_CHANNEL)
	{
		CollisionResponses.SetCollisionResponseContainer(ResponseToChannels_DEPRECATED);
	}
#endif // WITH_EDITORONLY_DATA

	// Load profile. If older version, please verify profile name first
	bool bNeedToVerifyProfile = (UEVersion < VER_UE4_COLLISION_PROFILE_SETTING) || 
		// or shape component needs to convert since we added profile
		(UEVersion < VER_UE4_SAVE_COLLISIONRESPONSE_PER_CHANNEL && Loader->IsA(UShapeComponent::StaticClass()));
	LoadProfileData(bNeedToVerifyProfile);

	// if profile isn't set, then fix up channel responses
	if( CollisionProfileName == UCollisionProfile::CustomCollisionProfileName ) 
	{
		if (UEVersion >= VER_UE4_SAVE_COLLISIONRESPONSE_PER_CHANNEL)
		{
			CollisionResponses.UpdateResponseContainerFromArray();
		}
	}
}

void FBodyInstance::ApplyMaterialToShape_AssumesLocked(const FPhysicsShapeHandle& InShape, UPhysicalMaterial* SimplePhysMat, const TArrayView<UPhysicalMaterial*>& ComplexPhysMats, const TArrayView<FPhysicalMaterialMaskParams>* ComplexPhysMatMasks)
{
	// If a triangle mesh, need to get array of materials...
	ECollisionShapeType GeomType = FPhysicsInterface::GetShapeType(InShape);
	if(GeomType == ECollisionShapeType::Trimesh)
	{
		if(ComplexPhysMats.Num())
		{
			if (ensure(ComplexPhysMatMasks))
			{
				FPhysicsInterface::SetMaterials(InShape, ComplexPhysMats, *ComplexPhysMatMasks);
			}
			else
			{
				FPhysicsInterface::SetMaterials(InShape, ComplexPhysMats);
			}
		}
		else
		{
			if(SimplePhysMat)
			{
				UE_LOGF(LogPhysics, Verbose, "FBodyInstance::ApplyMaterialToShape_AssumesLocked : PComplexMats is empty - falling back on simple physical material.");
				FPhysicsInterface::SetMaterials(InShape, {&SimplePhysMat, 1});
			}
			else
			{
				UE_LOGF(LogPhysics, Error, "FBodyInstance::ApplyMaterialToShape_AssumesLocked : PComplexMats is empty, and we do not have a valid simple material.");
			}
		}

	}
	// Simple shape, 
	else if(SimplePhysMat)
	{
		FPhysicsInterface::SetMaterials(InShape, {&SimplePhysMat, 1});
	}
	else
	{
		UE_LOGF(LogPhysics, Error, "FBodyInstance::ApplyMaterialToShape_AssumesLocked : No valid simple physics material found.");
	}
}

void FBodyInstance::ApplyMaterialToInstanceShapes_AssumesLocked(UPhysicalMaterial* SimplePhysMat, TArray<UPhysicalMaterial*>& ComplexPhysMats, const TArrayView<FPhysicalMaterialMaskParams>& ComplexPhysMatMasks)
{
	FBodyInstance* TheirBI = this;
	FBodyInstance* BIWithActor = TheirBI->WeldParent ? TheirBI->WeldParent : TheirBI;

	TArray<FPhysicsShapeHandle> AllShapes;
	BIWithActor->GetAllShapes_AssumesLocked(AllShapes);

	for(FPhysicsShapeHandle& Shape : AllShapes)
	{
		if(TheirBI->IsShapeBoundToBody(Shape))
		{
			FPhysicsCommand::ExecuteShapeWrite(BIWithActor, Shape, [&](const FPhysicsShapeHandle& InnerShape)
			{
				ApplyMaterialToShape_AssumesLocked(InnerShape, SimplePhysMat, ComplexPhysMats, &ComplexPhysMatMasks);
			});		
		}
	}
}

bool FBodyInstance::ValidateTransform(const FTransform &Transform, const FString& DebugName, const UBodySetup* Setup)
{
	return ValidateTransformScale(Transform, DebugName) 
		&& ValidateTransformMirror(Transform, DebugName, Setup->bGenerateMirroredCollision, Setup->bGenerateNonMirroredCollision)
		&& ValidateTransformNaN(Transform, DebugName, Setup->BoneName);
}

void FBodyInstance::InitDynamicProperties_AssumesLocked()
{
	UBodySetup* ResolvedSetup = GetBodySetup();
	if (!ResolvedSetup)
	{
		// This may be invalid following an undo if the BodySetup was a transient object (e.g. in Mesh Paint mode)
		// Just exit gracefully if so.
		return;
	}

	//QueryOnly bodies cannot become simulated at runtime. To do this they must change their CollisionEnabled which recreates the physics state
	//So early out to save a lot of useless work
	if (GetCollisionEnabled() == ECollisionEnabled::QueryOnly)
	{
		return;
	}

	if(FPhysicsInterface::IsDynamic(GetPhysicsActor()))
	{
		//A non simulated body may become simulated at runtime, so we need to compute its mass.
		//However, this is not supported for complexAsSimple since a trimesh cannot itself be simulated, it can only be used for collision of other simple shapes.
		if (ResolvedSetup->GetCollisionTraceFlag() != ECollisionTraceFlag::CTF_UseComplexAsSimple)
		{
			UpdateMassProperties();
			UpdateDampingProperties();
			SetMaxAngularVelocityInRadians(GetMaxAngularVelocityInRadians(), false, false);
			UpdateMaxDepenetrationVelocity();
			UpdateOneWayInteraction();
		}
		else
		{
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
			if (bSimulatePhysics && IsInGameThread())
			{
				if(UPrimitiveComponent* OwnerComponentInst = OwnerComponent.Get())
				{
					FMessageLog("PIE").Warning(FText::Format(LOCTEXT("SimComplexAsSimple", "Trying to simulate physics on ''{0}'' but it has ComplexAsSimple collision."),
						FText::FromString(GetPathNameSafe(OwnerComponentInst))));
				}
			}
#endif
		}

		FVector InitialLinVel;
		bool bComponentAwake = false;
		if (const FBodyInstancePhysicsCreationInputs* Cached = GetAsyncPhysicsCreationInputs())
		{
			InitialLinVel = Cached->ResolvedInitialLinearVelocity;
			bComponentAwake = Cached->bResolvedComponentAwake;
		}
		else
		{
			UPrimitiveComponent* OwnerComponentInst = OwnerComponent.Get();
			const AActor* OwningActor = OwnerComponentInst ? OwnerComponentInst->GetOwner() : nullptr;
			InitialLinVel = GetInitialLinearVelocity(OwningActor, bComponentAwake);
		}

		if (ShouldInstanceSimulatingPhysics())
		{
			FPhysicsInterface::SetLinearVelocity_AssumesLocked(GetPhysicsActor(), InitialLinVel);
		}

		FPhysicsInterface::SetSleepThresholdMultiplier_AssumesLocked(GetPhysicsActor(), GetSleepThresholdMultiplier());

		// @todo: implement sleep energy threshold
		float SleepEnergyThresh = FPhysicsInterface::GetSleepEnergyThreshold_AssumesLocked(GetPhysicsActor());
		SleepEnergyThresh *= GetSleepThresholdMultiplier();
		FPhysicsInterface::SetSleepEnergyThreshold_AssumesLocked(GetPhysicsActor(), SleepEnergyThresh);

		// PhysX specific dynamic parameters, not generically exposed to physics interface

		float StabilizationThreshold = FPhysicsInterface::GetStabilizationEnergyThreshold_AssumesLocked(GetPhysicsActor());
		StabilizationThreshold *= StabilizationThresholdMultiplier;
		FPhysicsInterface::SetStabilizationEnergyThreshold_AssumesLocked(GetPhysicsActor(), StabilizationThreshold);

		FPhysicsInterface::SetSolverPositionIterationCount_AssumesLocked(GetPhysicsActor(), GetPositionSolverIterationCount());
		FPhysicsInterface::SetSolverVelocityIterationCount_AssumesLocked(GetPhysicsActor(), GetVelocitySolverIterationCount());
		FPhysicsInterface::SetSolverProjectionIterationCount_AssumesLocked(GetPhysicsActor(), GetProjectionSolverIterationCount());

		CreateDOFLock();
		if(FPhysicsInterface::IsInScene(GetPhysicsActor()) && !IsRigidBodyKinematic_AssumesLocked(GetPhysicsActor()))
		{
			if(!bStartAwake && !bComponentAwake)
			{
				FPhysicsInterface::SetWakeCounter_AssumesLocked(GetPhysicsActor(), 0.0f);
				FPhysicsInterface::PutToSleep_AssumesLocked(GetPhysicsActor());

				// Initialising the velocity above will trigger a wake event on a sleeping actor so although the
				// actor will stay sleeping the user will receive a wake event unless we clear it prior to simulation
				FPhysicsInterface::ClearWakeEvent_AssumesLocked(GetPhysicsActor());
			}
		}

		FPhysicsInterface::SetInitialized_AssumesLocked(GetPhysicsActor(), true);
	}
}

void FBodyInstance::BuildBodyFilterData(FBodyCollisionFilterData& OutFilterData, const int32 ShapeIndex) const
{
	if (const FBodyInstancePhysicsCreationInputs* Cached = GetAsyncPhysicsCreationInputs())
	{
		OutFilterData = Cached->ResolvedCollisionData.CollisionFilterData;
	}
	else
	{
		FBodyInstance::BuildBodyFilterData(this, OwnerComponent.Get(), SourceObject.Get(), GetBodySetup(), OutFilterData, ShapeIndex);
	}
}

void FBodyInstance::BuildBodyFilterData(const FBodyInstance* InInstance, UPrimitiveComponent* InOwnerComp, UObject* InSourceObj, UBodySetup* InSetup, FBodyCollisionFilterData& OutFilterData, const int32 InShapeIndex)
{
	// this can happen in landscape height field collision component
	if(!InSetup)
	{
		return;
	}

	AActor* ActorOwner = Cast<AActor>(InSourceObj);
	if (!ActorOwner && InOwnerComp)
	{
		ActorOwner = InOwnerComp->GetOwner();
	}

	const bool bPhysicsStatic = !InOwnerComp || InOwnerComp->IsWorldGeometry();

	// Grab collision setting from body instance.
	// If we're looking at a specific shape, intersect it with the shape's specific collision setting.
	ECollisionEnabled::Type UseCollisionEnabled // this checks actor/component override.
		= InShapeIndex == INDEX_NONE
		? InInstance->GetCollisionEnabled()
		: CollisionEnabledIntersection(InInstance->GetCollisionEnabled(), InInstance->GetShapeCollisionEnabled(InShapeIndex));
	FCollisionResponseContainer UseResponse = InInstance->CollisionResponses.GetResponseContainer();
	bool bUseNotifyRBCollision = InInstance->bNotifyRigidBodyCollision;
	ECollisionChannel UseChannel = InInstance->ObjectType;


	bool bUseContactModification = InInstance->bContactModification;

	// @TODO: The skel mesh really shouldn't be the (pseudo-)authority here on the body's collision.
	//        This block should ultimately be removed, and outside of this (in the skel component) we
	//        should configure the bodies to reflect this desired behavior.
	if(USkeletalMeshComponent* SkelMeshComp = Cast<USkeletalMeshComponent>(InOwnerComp))
	{
		if (CVarEnableDynamicPerBodyFilterHacks.GetValueOnAnyThread() && InInstance->bHACK_DisableCollisionResponse)
		{
			UseResponse.SetAllChannels(ECR_Ignore);
			UseCollisionEnabled = ECollisionEnabled::PhysicsOnly;
		}
		else if(InSetup->CollisionReponse == EBodyCollisionResponse::BodyCollision_Enabled)
		{
			UseResponse.SetAllChannels(ECR_Block);
		}
		else if(InSetup->CollisionReponse == EBodyCollisionResponse::BodyCollision_Disabled)
		{
			UseResponse.SetAllChannels(ECR_Ignore);
			UseCollisionEnabled = ECollisionEnabled::PhysicsOnly;		// this will prevent object traces hitting this as well
		}

		const bool bDisableSkelComponentOverride = CVarEnableDynamicPerBodyFilterHacks.GetValueOnAnyThread() && InInstance->bHACK_DisableSkelComponentFilterOverriding;
		if (bDisableSkelComponentOverride)
		{
			// if we are disabling the skeletal component override, we want the original body instance collision response
			// this is to allow per body instance collision filtering instead of taking the data from the skeletal mesh
			UseResponse = InInstance->CollisionResponses.GetResponseContainer();
		}
		else
		{
			UseChannel = SkelMeshComp->GetCollisionObjectType();
			UseResponse = FCollisionResponseContainer::CreateMinContainer(UseResponse, SkelMeshComp->BodyInstance.CollisionResponses.GetResponseContainer());
		}

		bUseNotifyRBCollision = bUseNotifyRBCollision && SkelMeshComp->BodyInstance.bNotifyRigidBodyCollision;
	}

	if (InShapeIndex != INDEX_NONE)
	{
		// If we're looking at a specific shape, take the intersection of its collision responses
		// with the base primitive component's collision responses.
		FCollisionResponseContainer ShapeResponse = InInstance->GetShapeResponseToChannels(InShapeIndex, UseResponse);
		UseResponse = FCollisionResponseContainer::CreateMinContainer(UseResponse, ShapeResponse);
	}

#if WITH_EDITOR
	// if no collision, but if world wants to enable trace collision for components, allow it
	if((UseCollisionEnabled == ECollisionEnabled::NoCollision) && ActorOwner && (ActorOwner->IsA(AVolume::StaticClass()) == false))
	{
		UWorld* World = ActorOwner->GetWorld();
		if(World && World->bEnableTraceCollision && InOwnerComp &&
		   (InOwnerComp->IsA(UStaticMeshComponent::StaticClass()) || InOwnerComp->IsA(USkeletalMeshComponent::StaticClass()) || InOwnerComp->IsA(UBrushComponent::StaticClass())))
		{
			//UE_LOGF(LogPhysics, Warning, "Enabling collision %ls : %ls", *GetNameSafe(Owner), *GetNameSafe(InOwnerComp));
			// clear all other channel just in case other people using those channels to do something
			UseResponse.SetAllChannels(ECR_Ignore);
			UseCollisionEnabled = ECollisionEnabled::QueryOnly;
		}
	}
#endif

	const bool bUseComplexAsSimple = (InSetup->GetCollisionTraceFlag() == CTF_UseComplexAsSimple);
	const bool bUseSimpleAsComplex = (InSetup->GetCollisionTraceFlag() == CTF_UseSimpleAsComplex);

	if(UseCollisionEnabled != ECollisionEnabled::NoCollision)
	{
		// CCD is determined by root body in case of welding
		bool bRootCCD = (InInstance->WeldParent != nullptr) ? InInstance->WeldParent->bUseCCD : InInstance->bUseCCD;

		uint32 SourceObjectID = InSourceObj ? InSourceObj->GetUniqueID() : ActorOwner ? ActorOwner->GetUniqueID() : 0;
		uint32 CompID = (InOwnerComp != nullptr) ? InOwnerComp->GetUniqueID() : 0;

		FPhysicsFilterBuilder Builder;
		Builder.SetOwnerID(SourceObjectID);
		Builder.SetComponentID(CompID);
		Builder.SetCollisionChannelIndex(UseChannel);
		Builder.SetResponses(UseResponse);
		Builder.SetMaskFilter(InInstance->MaskFilter);
		Builder.SetFlags(Chaos::EFilterFlags::CCD, bRootCCD && !bPhysicsStatic);
		Builder.SetFlags(Chaos::EFilterFlags::ContactNotify, bUseNotifyRBCollision);
		Builder.SetFlags(Chaos::EFilterFlags::StaticShape, bPhysicsStatic);
		Builder.SetFlags(Chaos::EFilterFlags::ModifyContacts, bUseContactModification);

		OutFilterData.FilterInstanceData = Builder.BuildInstanceData();

		// Build shape filter variations for complex and simple
		Builder.SetFlags(Chaos::EFilterFlags::SimpleCollision, true);
		Builder.SetFlags(Chaos::EFilterFlags::ComplexCollision, bUseSimpleAsComplex);
		OutFilterData.SimpleShapeFilterData = Builder.BuildShapeFilterData();

		Builder.SetFlags(Chaos::EFilterFlags::ComplexCollision, true);
		Builder.SetFlags(Chaos::EFilterFlags::SimpleCollision, bUseComplexAsSimple);
		OutFilterData.ComplexShapeFilterData = Builder.BuildShapeFilterData();
	}
}

void FBodyInstance::InitStaticBodies(const TArray<FBodyInstance*>& Bodies, const TArray<FTransform>& Transforms, UBodySetup* BodySetup, UPrimitiveComponent* PrimitiveComp, FPhysScene* InRBScene)
{
	SCOPE_CYCLE_COUNTER(STAT_StaticInitBodies);

	check(BodySetup);
	check(InRBScene);
	check(Bodies.Num() > 0);
	check(IsInGameThread());

	static TArray<FBodyInstance*> BodiesStatic;
	static TArray<FTransform> TransformsStatic;

	check(BodiesStatic.Num() == 0);
	check(TransformsStatic.Num() == 0);

	BodiesStatic = Bodies;
	TransformsStatic = Transforms;

	FInitBodiesHelper<true> InitBodiesHelper(BodiesStatic, TransformsStatic, BodySetup, PrimitiveComp, InRBScene, FInitBodySpawnParams(PrimitiveComp), FPhysicsAggregateHandle());
	InitBodiesHelper.InitBodies();

	BodiesStatic.Reset();
	TransformsStatic.Reset();
}

void FBodyInstance::InitStaticBodies(TArray<FBodyInstance*>&& Bodies, TArray<FTransform>&& Transforms, UBodySetup* BodySetup, UPrimitiveComponent* PrimitiveComp, FPhysScene* InRBScene, IPhysicsBodyInstanceOwner* InBodyInstanceOwner)
{
	SCOPE_CYCLE_COUNTER(STAT_StaticInitBodies);
	check(BodySetup);
	check(InRBScene);
	check(Bodies.Num() > 0);
	check(!InBodyInstanceOwner || InBodyInstanceOwner->IsStaticPhysics());
	FInitBodiesHelperWithData<true> InitBodiesHelper(MoveTemp(Bodies), MoveTemp(Transforms), BodySetup, PrimitiveComp, InRBScene, FInitBodySpawnParams(PrimitiveComp), FPhysicsAggregateHandle(), InBodyInstanceOwner);
	InitBodiesHelper.InitBodies();
}

int32 SimCollisionEnabled = 1;
FAutoConsoleVariableRef CVarSimCollisionEnabled(TEXT("p.SimCollisionEnabled"), SimCollisionEnabled, TEXT("If 0 no sim collision will be used"));

void FBodyInstance::BuildBodyCollisionFlags(FBodyCollisionFlags& OutFlags, ECollisionEnabled::Type UseCollisionEnabled, bool bUseComplexAsSimple)
{
	OutFlags.bEnableQueryCollision = false;
	OutFlags.bEnableSimCollisionSimple = false;
	OutFlags.bEnableSimCollisionComplex = false;
	OutFlags.bEnableProbeCollision = false;

	if(UseCollisionEnabled != ECollisionEnabled::NoCollision)
	{
		// Query collision
		OutFlags.bEnableQueryCollision = CollisionEnabledHasQuery(UseCollisionEnabled);

		// Sim collision
		const bool bProbe = CollisionEnabledHasProbe(UseCollisionEnabled);
		const bool bSimCollision = SimCollisionEnabled && (bProbe || CollisionEnabledHasPhysics(UseCollisionEnabled));

		// Enable sim collision
		if(bSimCollision)
		{
			// Objects marked as probes use sim collision, but don't actually have physical reactions
			OutFlags.bEnableProbeCollision = bProbe;

			// We use simple sim collision even if we also use complex sim collision?
			OutFlags.bEnableSimCollisionSimple = true;
			
			// on dynamic objects and objects which don't use complex as simple, tri mesh not used for sim
			if(bUseComplexAsSimple)
			{
				OutFlags.bEnableSimCollisionComplex = true;
			}
		}
	}
}

inline void FBodyInstance::SetSolverAsyncDeltaTime(const float NewSolverAsyncDeltaTime)
{
	bOverrideSolverAsyncDeltaTime = true;
	SolverAsyncDeltaTime = NewSolverAsyncDeltaTime;
	UpdateSolverAsyncDeltaTime();
}

void FBodyInstance::UpdateSolverAsyncDeltaTime()
{
	if (bOverrideSolverAsyncDeltaTime && !bEnableOverrideSolverDeltaTime)
	{
		UE_LOGF(LogPhysics, Warning, "FBodyInstance::SolverAsyncDeltaTime : Ignoring parameter because overriden by p.EnableOverrideSolverDeltaTime");
		return;
	}

	if (bOverrideSolverAsyncDeltaTime)
	{
		if (SolverAsyncDeltaTime < 0.f)
		{
			UE_LOGF(LogPhysics, Error, "FBodyInstance::SolverAsyncDeltaTime : Value must be greater than 0");			
			return;
		}

		Chaos::FPhysicsSolverBase* Solver = GetPhysicsActor()->GetSolverBase();

		if (!Solver->IsUsingAsyncResults())
		{
			UE_LOGF(LogPhysics, Warning, "FBodyInstance::SolverAsyncDeltaTime : Ignoring parameter because solver is not setup to use async results");
			return;
		}
		else
		{
			float OldAsyncDeltaTime = Solver->GetAsyncDeltaTime();

			// set async delta time to minimum of delta time and what this body wants
			Solver->EnableAsyncMode(FMath::Min<float>(Solver->GetAsyncDeltaTime(), SolverAsyncDeltaTime));
		}
	}
}

void FBodyInstance::UpdateInterpolateWhenSubStepping()
{
	if(UPhysicsSettings::Get()->bSubstepping)
	{
		// We interpolate based around our current collision enabled flag
		ECollisionEnabled::Type UseCollisionEnabled = ECollisionEnabled::NoCollision;
		if(OwnerComponent.IsValid() && OwnerComponent.Get()->GetBodyInstance() != this)
		{
			UseCollisionEnabled = OwnerComponent->GetCollisionEnabled();
		}
		else
		{
			UseCollisionEnabled = GetCollisionEnabled();
		}
	
		bInterpolateWhenSubStepping = CollisionEnabledHasPhysics(UseCollisionEnabled);

		// If we have a weld parent we should take into account that too as that may be simulating while we are not
		if(WeldParent)
		{
			// Potentially recurse here
			WeldParent->UpdateInterpolateWhenSubStepping();
			bInterpolateWhenSubStepping |= WeldParent->bInterpolateWhenSubStepping;
		}
	}
}

void FBodyInstance::ExecuteOnCalculateCustomProjection(FTransform& WorldTM) const
{
	if (BodyInstanceDelegates.IsValid())
	{
		BodyInstanceDelegates->OnCalculateCustomProjection.ExecuteIfBound(this, WorldTM);
	}
}

FCalculateCustomProjection& FBodyInstance::OnCalculateCustomProjection()
{
	if (!BodyInstanceDelegates.IsValid())
	{
		BodyInstanceDelegates.Reset(new FBodyInstanceDelegates);
	}

	return BodyInstanceDelegates->OnCalculateCustomProjection;
}

FRecalculatedMassProperties& FBodyInstance::OnRecalculatedMassProperties()
{
	if (!BodyInstanceDelegates.IsValid())
	{
		BodyInstanceDelegates.Reset(new FBodyInstanceDelegates);
	}

	return BodyInstanceDelegates->OnRecalculatedMassProperties;
}

const FBodyInstancePhysicsCreationInputs* FBodyInstance::GetAsyncPhysicsCreationInputs() const
{
	if (!IsInGameThread() && !IsInParallelGameThread() && PhysicsCreationInputs.IsValid())
	{
		return PhysicsCreationInputs.Get();
	}
	return nullptr;
}

const FBodyInstancePhysicsCreationInputsPtr& FBodyInstance::CachePhysicsCreationInputs(UPrimitiveComponent* InOwnerComponent, IPhysicsBodyInstanceOwner* InBodyInstanceOwner)
{
	check(IsInGameThread() || IsInParallelGameThread());
	check(!PhysicsCreationInputs.IsValid());

	// UPrimitiveComponent implements IPhysicsBodyInstanceOwner, so use it as fallback
	if (!InBodyInstanceOwner)
	{
		InBodyInstanceOwner = InOwnerComponent;
	}
	check(InBodyInstanceOwner);

	FBodyInstancePhysicsCreationInputsPtr NewPhysicsCreationInputs = MakeShared<FBodyInstancePhysicsCreationInputs, ESPMode::ThreadSafe>();
	UBodySetup* OwnerBodySetup = InBodyInstanceOwner->GetPhysicsBodySetup();
	NewPhysicsCreationInputs->ResolvedBodySetup = OwnerBodySetup;
	NewPhysicsCreationInputs->ResolvedSimpleMaterial = FBodyInstance::GetSimplePhysicalMaterial(this, TWeakObjectPtr<UPrimitiveComponent>(), OwnerBodySetup, InBodyInstanceOwner);
	FBodyInstance::GetComplexPhysicalMaterials(this, TWeakObjectPtr<UPrimitiveComponent>(), NewPhysicsCreationInputs->ResolvedComplexMaterials, &NewPhysicsCreationInputs->ResolvedComplexMasks, InBodyInstanceOwner);

	// Pre-compute collision filter data (avoids OwnerComponent/SourceObject weak ptr access on worker thread)
	UObject* SourceObj = UE::Private::GetSourceObject(InOwnerComponent, InBodyInstanceOwner);
	FBodyInstance::BuildBodyFilterData(this, InOwnerComponent, SourceObj, OwnerBodySetup, NewPhysicsCreationInputs->ResolvedCollisionData.CollisionFilterData);
	const bool bUseComplexAsSimple = OwnerBodySetup ? OwnerBodySetup->GetCollisionTraceFlag() == CTF_UseComplexAsSimple : false;
	FBodyInstance::BuildBodyCollisionFlags(NewPhysicsCreationInputs->ResolvedCollisionData.CollisionFlags, GetCollisionEnabled(), bUseComplexAsSimple);


	// Pre-compute initial velocity for InitDynamicProperties (avoids OwnerComponent dereference on worker thread)
	const AActor* OwningActor = InOwnerComponent ? InOwnerComponent->GetOwner() : nullptr;
	bool bComponentAwake = false;
	NewPhysicsCreationInputs->ResolvedInitialLinearVelocity = GetInitialLinearVelocity(OwningActor, bComponentAwake);
	NewPhysicsCreationInputs->bResolvedComponentAwake = bComponentAwake;

	// Cache component pointer as a map key only (rooted by the async job queue, safe to use as key on worker thread)
	NewPhysicsCreationInputs->OwnerComponentMapKey = InOwnerComponent;

	PhysicsCreationInputs = NewPhysicsCreationInputs;
	return PhysicsCreationInputs;
}

void FBodyInstancePhysicsCreationInputs::GetReferencedObjects(TSet<UObject*>& OutReferencedObjects)
{
	if (ResolvedBodySetup)
	{
		OutReferencedObjects.Add(ResolvedBodySetup);
	}

	if (ResolvedSimpleMaterial)
	{
		OutReferencedObjects.Add(ResolvedSimpleMaterial);
	}

	for (UPhysicalMaterial* ComplexMaterial : ResolvedComplexMaterials)
	{
		if (ComplexMaterial)
		{
			OutReferencedObjects.Add(ComplexMaterial);
		}
	}

	for (const FPhysicalMaterialMaskParams& Mask : ResolvedComplexMasks)
	{
		if (Mask.PhysicalMaterialMap)
		{
			OutReferencedObjects.Add(Mask.PhysicalMaterialMap);
		}
		if (Mask.PhysicalMaterialMask)
		{
			OutReferencedObjects.Add(Mask.PhysicalMaterialMask);
		}
	}
}

bool FBodyInstanceAsyncPhysicsTickHandle::IsValid() const
{
	return Proxy && Proxy->GetPhysicsThreadAPI() != nullptr;
}

Chaos::FRigidBodyHandle_Internal* FBodyInstanceAsyncPhysicsTickHandle::operator->()
{
	return Proxy->GetPhysicsThreadAPI();
}


////////////////////////////////////////////////////////////////////////////
// FBodyInstanceEditorHelpers

#if WITH_EDITOR

void FBodyInstanceEditorHelpers::EnsureConsistentMobilitySimulationSettingsOnPostEditChange(UPrimitiveComponent* Component, FPropertyChangedEvent& PropertyChangedEvent)
{
	if (FProperty* PropertyThatChanged = PropertyChangedEvent.Property)
	{
		const FName PropertyName = PropertyThatChanged->GetFName();

		// Automatically change collision profile based on mobility and physics settings (if it is currently one of the default profiles)
		const bool bMobilityChanged = PropertyName == GET_MEMBER_NAME_CHECKED(USceneComponent, Mobility);
		const bool bSimulatePhysicsChanged = PropertyName == GET_MEMBER_NAME_CHECKED(FBodyInstance, bSimulatePhysics);

		if (bMobilityChanged || bSimulatePhysicsChanged)
		{
			// If we enabled physics simulation, but we are not marked movable, do that for them
			if (bSimulatePhysicsChanged && Component->BodyInstance.bSimulatePhysics && (Component->Mobility != EComponentMobility::Movable))
			{
				Component->SetMobility(EComponentMobility::Movable);
			}
			// If we made the component no longer movable, but simulation was enabled, disable that for them
			else if (bMobilityChanged && (Component->Mobility != EComponentMobility::Movable) && Component->BodyInstance.bSimulatePhysics)
			{
				Component->BodyInstance.bSimulatePhysics = false;
			}

			// If the collision profile is one of the 'default' ones for a StaticMeshActor, make sure it is the correct one
			// If user has changed it to something else, don't touch it
			const FName CurrentProfileName = Component->BodyInstance.GetCollisionProfileName();
			if ((CurrentProfileName == UCollisionProfile::BlockAll_ProfileName) ||
				(CurrentProfileName == UCollisionProfile::BlockAllDynamic_ProfileName) ||
				(CurrentProfileName == UCollisionProfile::PhysicsActor_ProfileName))
			{
				if (Component->Mobility == EComponentMobility::Movable)
				{
					if (Component->BodyInstance.bSimulatePhysics)
					{
						Component->SetCollisionProfileName(UCollisionProfile::PhysicsActor_ProfileName);
					}
					else
					{
						Component->SetCollisionProfileName(UCollisionProfile::BlockAllDynamic_ProfileName);
					}
				}
				else
				{
					Component->SetCollisionProfileName(UCollisionProfile::BlockAll_ProfileName);
				}
			}
		}
	}
}

#endif // WITH_EDITOR

#undef LOCTEXT_NAMESPACE

