// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	BodySetup.cpp
=============================================================================*/ 

#include "PhysicsEngine/BodySetup.h"
#include "BodySetupEnums.h"
#include "Engine/Engine.h"
#include "Engine/StaticMesh.h"
#include "Engine/SkeletalMesh.h"
#include "Components/SkinnedMeshComponent.h"
#include "Engine/World.h"
#include "PhysicsEngine/PhysicsSettings.h"
#include "Interfaces/ITargetPlatform.h"
#include "Interfaces/ITargetPlatformManagerModule.h"
#include "Animation/AnimStats.h"
#include "DerivedDataCacheInterface.h"
#include "Physics/PhysicsInterfaceTypes.h"
#include "UObject/UObjectIterator.h"
#include "Components/SplineMeshComponent.h"
#include "PhysicsEngine/BoxElem.h"
#include "UObject/FortniteReleaseBranchCustomObjectVersion.h"

#include "Chaos/Convex.h"


#include "PhysicalMaterials/PhysicalMaterial.h"
#include "PhysicsEngine/ConvexElem.h"
#include "ProfilingDebugging/CookStats.h"
#include "PhysicsEngine/LevelSetElem.h"
#include "Serialization/MemoryWriter.h"
#include "PhysicsEngine/SphereElem.h"
#include "UObject/AnimPhysObjectVersion.h"

#include "Chaos/TriangleMeshImplicitObject.h"
#include "Experimental/ChaosDerivedData.h"
#include "Physics/Experimental/ChaosDerivedDataReader.h"
#include "Chaos/CollisionConvexMesh.h"
#include "PhysicsEngine/Experimental/ChaosCooking.h"
#include "PhysicsEngine/SphylElem.h"
#include "PhysicsEngine/TaperedCapsuleElem.h"
#include "PhysicsEngine/MLLevelSetElem.h"
#include "PhysicsEngine/SkinnedTriangleMeshElem.h"

#if WITH_EDITOR
#include "Misc/DataValidation.h"
#include "PhysicsEngine/BodySetupObjectTextFactory.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(BodySetup)

/** Enable to verify that the cooked data matches the source data as we cook it */
#define VERIFY_COOKED_PHYS_DATA 0

#define LOCTEXT_NAMESPACE "PhysicsAsset"


FCookBodySetupInfo::FCookBodySetupInfo() :
	bCookNonMirroredConvex(false),
	bCookMirroredConvex(false),
	bConvexDeformableMesh(false),
	bCookTriMesh(false),
	bSupportUVFromHitResults(false),
	bSupportFaceRemap(false),
	bSupportFaceRemapOnMeshBVH(true),
	bSupportVertexRemap(true),
	bTriMeshError(false)
{
}

UBodySetup::UBodySetup(FVTableHelper& Helper)
	: Super(Helper)
{
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
UBodySetup::~UBodySetup() {}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

#if ENABLE_COOK_STATS
namespace BodySetupCookStats
{
	static FCookStats::FDDCResourceUsageStats UsageStats;
	static FCookStatsManager::FAutoRegisterCallback RegisterCookStats([](FCookStatsManager::AddStatFuncRef AddStat)
	{
		UsageStats.LogStats(AddStat, TEXT("BodySetup.Usage"), TEXT(""));
	});
}
#endif

DEFINE_STAT(STAT_PhysXCooking);

bool IsRuntimeCookingEnabled()
{
	return false;
}

// CVars
ENGINE_API TAutoConsoleVariable<float> CVarContactOffsetFactor(
	TEXT("p.ContactOffsetFactor"),
	-1.f,
	TEXT("Multiplied by min dimension of object to calculate how close objects get before generating contacts. < 0 implies use project settings. Default: 0.01"),
	ECVF_Default);

ENGINE_API TAutoConsoleVariable<float> CVarMaxContactOffset(
	TEXT("p.MaxContactOffset"),
	-1.f,
	TEXT("Max value of contact offset, which controls how close objects get before generating contacts. < 0 implies use project settings. Default: 1.0"),
	ECVF_Default);

#if WITH_EDITOR
ENGINE_API TAutoConsoleVariable<int32> CVarBodySetupSkipDDCThreshold(
	TEXT("p.BodySetupSkipDDCThreshold"),
	16384,
	TEXT("Enables skipping the DDC for body setups with vertice count under threshold. Default: 16384"),
	ECVF_Default);
#endif

void FBodySetupUVInfo::GetResourceSizeEx(FResourceSizeEx& CumulativeResourceSize) const
{
	CumulativeResourceSize.AddDedicatedSystemMemoryBytes(IndexBuffer.GetAllocatedSize());
	CumulativeResourceSize.AddDedicatedSystemMemoryBytes(VertPositions.GetAllocatedSize());

	for (int32 ChannelIdx = 0; ChannelIdx < VertUVs.Num(); ChannelIdx++)
	{
		CumulativeResourceSize.AddDedicatedSystemMemoryBytes(VertUVs[ChannelIdx].GetAllocatedSize());
	}

	CumulativeResourceSize.AddDedicatedSystemMemoryBytes(VertUVs.GetAllocatedSize());
}

DEFINE_LOG_CATEGORY(LogPhysics);
UBodySetup::UBodySetup(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bConsiderForBounds = true;
	bMeshCollideAll = false;
	bFailedToCreatePhysicsMeshes = false;
	bHasCookedCollisionData = true;
	bNeverNeedsCookedCollisionData = false;
	bGenerateMirroredCollision = true;
	bGenerateNonMirroredCollision = true;
	bSupportUVsAndFaceRemap = false;
	bSupportFaceRemapOnMeshBVH = true;
	bSupportVertexRemap = true;
	DefaultInstance.SetObjectType(ECC_PhysicsBody);
#if WITH_EDITORONLY_DATA
	BuildScale_DEPRECATED = 1.0f;
#endif
	BuildScale3D = FVector::OneVector;
	SetFlags(RF_Transactional);
	bSharedCookedData = false;
	bHasCookedFormatProvider = false;
	CookedFormatProvider = nullptr;
	CurrentCookHelper = nullptr;
}

void UBodySetup::CopyBodyPropertiesFrom(const UBodySetup* FromSetup)
{
	AggGeom = FromSetup->AggGeom;

	// clear pointers copied from other BodySetup, as 
	for (int32 i = 0; i < AggGeom.ConvexElems.Num(); i++)
	{
		FKConvexElem& ConvexElem = AggGeom.ConvexElems[i];
		ConvexElem.ResetChaosConvexMesh();
	}

	DefaultInstance.CopyBodyInstancePropertiesFrom(&FromSetup->DefaultInstance);
	PhysMaterial = FromSetup->PhysMaterial;
	PhysicsType = FromSetup->PhysicsType;
	bDoubleSidedGeometry = FromSetup->bDoubleSidedGeometry;
	CollisionTraceFlag = FromSetup->CollisionTraceFlag;
}

void UBodySetup::AddCollisionFrom(const FKAggregateGeom& FromAggGeom)
{
	static_assert((int)EAggCollisionShape::Unknown == 9, "Did you forget to add a new AggCollisionShape type to this method?");

	AggGeom.SphereElems.Append(FromAggGeom.SphereElems);
	AggGeom.BoxElems.Append(FromAggGeom.BoxElems);
	AggGeom.SphylElems.Append(FromAggGeom.SphylElems);

	// Remember how many convex we already have
	int32 FirstNewConvexIdx = AggGeom.ConvexElems.Num();
	// copy convex
	AggGeom.ConvexElems.Append(FromAggGeom.ConvexElems);
	// clear pointers on convex elements
	for (int32 i = FirstNewConvexIdx; i < AggGeom.ConvexElems.Num(); i++)
	{
		FKConvexElem& ConvexElem = AggGeom.ConvexElems[i];
		ConvexElem.ResetChaosConvexMesh();
	}

	AggGeom.TaperedCapsuleElems.Append(FromAggGeom.TaperedCapsuleElems);
	AggGeom.LevelSetElems.Append(FromAggGeom.LevelSetElems);
	AggGeom.SkinnedLevelSetElems.Append(FromAggGeom.SkinnedLevelSetElems);
	AggGeom.MLLevelSetElems.Append(FromAggGeom.MLLevelSetElems);
	AggGeom.SkinnedTriangleMeshElems.Append(FromAggGeom.SkinnedTriangleMeshElems);
}

namespace
{
	template <typename TElem>
	bool AddCollisionElemFrom_Helper(const TArray<TElem>& FromElems, const int32 ElemIndex, TArray<TElem>& ToElems)
	{
		if (FromElems.IsValidIndex(ElemIndex))
		{
			ToElems.Add(FromElems[ElemIndex]);
			return true;
		}
		return false;
	}
}

bool UBodySetup::AddCollisionElemFrom(const FKAggregateGeom& FromAggGeom, const EAggCollisionShape::Type ShapeType, const int32 ElemIndex)
{
	static_assert((int)EAggCollisionShape::Unknown == 9, "Did you forget to add a new AggCollisionShape type to this method?");
	switch (ShapeType)
	{
	case EAggCollisionShape::Sphere:
		return AddCollisionElemFrom_Helper(FromAggGeom.SphereElems, ElemIndex, AggGeom.SphereElems);
	case EAggCollisionShape::Box:
		return AddCollisionElemFrom_Helper(FromAggGeom.BoxElems, ElemIndex, AggGeom.BoxElems);
	case EAggCollisionShape::Sphyl:
		return AddCollisionElemFrom_Helper(FromAggGeom.SphylElems, ElemIndex, AggGeom.SphylElems);
	case EAggCollisionShape::Convex:
		if (AddCollisionElemFrom_Helper(FromAggGeom.ConvexElems, ElemIndex, AggGeom.ConvexElems))
		{
			AggGeom.ConvexElems.Last().ResetChaosConvexMesh();
			return true;
		}
		return false;
	case EAggCollisionShape::TaperedCapsule:
		return AddCollisionElemFrom_Helper(FromAggGeom.TaperedCapsuleElems, ElemIndex, AggGeom.TaperedCapsuleElems);
	case EAggCollisionShape::LevelSet:
		return AddCollisionElemFrom_Helper(FromAggGeom.LevelSetElems, ElemIndex, AggGeom.LevelSetElems);
	case EAggCollisionShape::SkinnedLevelSet:
		return AddCollisionElemFrom_Helper(FromAggGeom.SkinnedLevelSetElems, ElemIndex, AggGeom.SkinnedLevelSetElems);
	case EAggCollisionShape::MLLevelSet:
		return AddCollisionElemFrom_Helper(FromAggGeom.MLLevelSetElems, ElemIndex, AggGeom.MLLevelSetElems);
	case EAggCollisionShape::SkinnedTriangleMesh:
		return AddCollisionElemFrom_Helper(FromAggGeom.SkinnedTriangleMeshElems, ElemIndex, AggGeom.SkinnedTriangleMeshElems);
	default:
		checkNoEntry();
	}
	return false;
}

void UBodySetup::GetCookInfo(FCookBodySetupInfo& OutCookInfo, EPhysXMeshCookFlags InCookFlags) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UBodySetup::GetCookInfo);

	OutCookInfo.OuterDebugName = GetOuter()->GetPathName();
	OutCookInfo.bConvexDeformableMesh = false;

	// Cook convex meshes, but only if we are not forcing complex collision to be used as simple collision as well
	if (GetCollisionTraceFlag() != CTF_UseComplexAsSimple && AggGeom.ConvexElems.Num() > 0)
	{
		OutCookInfo.bCookNonMirroredConvex = bGenerateNonMirroredCollision;
		OutCookInfo.bCookMirroredConvex = bGenerateMirroredCollision;
		for (int32 ElementIndex = 0; ElementIndex < AggGeom.ConvexElems.Num(); ElementIndex++)
		{
			const FKConvexElem& ConvexElem = AggGeom.ConvexElems[ElementIndex];
			const int32 NumVertices = ConvexElem.VertexData.Num();

			TArray<FVector>* NonMirroredConvexVertices = nullptr;
			TArray<FVector>* MirroredConvexVertices = nullptr;

			if (bGenerateNonMirroredCollision)
			{
				OutCookInfo.NonMirroredConvexVertices.AddDefaulted();
				NonMirroredConvexVertices = &OutCookInfo.NonMirroredConvexVertices.Last();
				NonMirroredConvexVertices->AddUninitialized(NumVertices);
			}

			if (bGenerateMirroredCollision)
			{
				OutCookInfo.MirroredConvexVertices.AddDefaulted();
				MirroredConvexVertices = &OutCookInfo.MirroredConvexVertices.Last();
				MirroredConvexVertices->AddUninitialized(NumVertices);
			}

			FTransform ConvexTransform = ConvexElem.GetTransform();
			if (!ConvexTransform.IsValid())
			{
				UE_LOGF(LogPhysics, Warning, "UBodySetup::GetCookInfoConvex: [%ls] ConvexElem[%d] has invalid transform", *GetPathNameSafe(GetOuter()), ElementIndex);
				ConvexTransform = FTransform::Identity;
			}

			// Transform verts from element to body space, and mirror if desired
			for (int32 VertIdx = 0; VertIdx< NumVertices; VertIdx++)
			{
				FVector BodySpaceVert = ConvexTransform.TransformPosition(ConvexElem.VertexData[VertIdx]);
				if (NonMirroredConvexVertices)
				{
					(*NonMirroredConvexVertices)[VertIdx] = BodySpaceVert;
				}

				if (MirroredConvexVertices)
				{
					(*MirroredConvexVertices)[VertIdx] = BodySpaceVert * FVector(-1, 1, 1);
				}
			}

			// Get cook flags to use
			OutCookInfo.ConvexCookFlags = InCookFlags;
			OutCookInfo.bConvexDeformableMesh = GetOuter()->IsA(USplineMeshComponent::StaticClass());
		}
	}
	else
	{
		OutCookInfo.bCookNonMirroredConvex = false;
		OutCookInfo.bCookMirroredConvex = false;
	}

	// Cook trimesh, but only if we do not force simple collision to be used as complex collision as well
	const bool bUsingAllTriData = bMeshCollideAll;
	OutCookInfo.bCookTriMesh = false;
	OutCookInfo.bTriMeshError = false;

	UObject* CDPObj = GetOuter();
	IInterface_CollisionDataProvider* CDP = Cast<IInterface_CollisionDataProvider>(CDPObj);
	
	if (GetCollisionTraceFlag() != CTF_UseSimpleAsComplex && CDP && CDP->ContainsPhysicsTriMeshData(bUsingAllTriData))
	{
		OutCookInfo.bCookTriMesh = CDP->GetPhysicsTriMeshData(&OutCookInfo.TriangleMeshDesc, bUsingAllTriData);
		const FTriMeshCollisionData& TriangleMeshDesc = OutCookInfo.TriangleMeshDesc;

		if (OutCookInfo.bCookTriMesh)
		{
			// If any of the below checks gets hit this usually means 
			// IInterface_CollisionDataProvider::ContainsPhysicsTriMeshData did not work properly.
			const int32 NumIndices = TriangleMeshDesc.Indices.Num();
			const int32 NumVerts = TriangleMeshDesc.Vertices.Num();
			if (NumIndices == 0 || NumVerts == 0 || TriangleMeshDesc.MaterialIndices.Num() > NumIndices)
			{
				UE_LOGF(LogPhysics, Warning, "UBodySetup::GetCookInfo: Triangle data from '%ls' invalid (%d verts, %d indices).", *CDPObj->GetPathName(), NumVerts, NumIndices);
				OutCookInfo.bTriMeshError = true;
			}

			// Set up cooking flags
			EPhysXMeshCookFlags CookFlags = InCookFlags;
			OutCookInfo.TriMeshCookFlags = CookFlags;

			OutCookInfo.bSupportFaceRemap = bSupportUVsAndFaceRemap;
			OutCookInfo.bSupportFaceRemapOnMeshBVH = bSupportFaceRemapOnMeshBVH;
			OutCookInfo.bSupportVertexRemap = bSupportVertexRemap;
		}
		else
		{
			UE_LOGF(LogPhysics, Warning, "UBodySetup::GetCookInfo: ContainsPhysicsTriMeshData returned true, but GetPhysicsTriMeshData returned false. This inconsistency should be fixed for asset '%ls'", *CDPObj->GetPathName());
		}
	}

	OutCookInfo.bSupportUVFromHitResults = UPhysicsSettings::Get()->bSupportUVFromHitResults || bSupportUVsAndFaceRemap;


}

void FBodySetupUVInfo::FillFromTriMesh(const FTriMeshCollisionData& TriangleMeshDesc)
{
	// Store index buffer
	const int32 NumVerts = TriangleMeshDesc.Vertices.Num();
	const int32 NumTris = TriangleMeshDesc.Indices.Num();
	IndexBuffer.Empty();
	IndexBuffer.AddUninitialized(NumTris * 3);
	for (int32 TriIdx = 0; TriIdx < TriangleMeshDesc.Indices.Num(); TriIdx++)
	{
		IndexBuffer[TriIdx * 3 + 0] = TriangleMeshDesc.Indices[TriIdx].v0;
		IndexBuffer[TriIdx * 3 + 1] = TriangleMeshDesc.Indices[TriIdx].v1;
		IndexBuffer[TriIdx * 3 + 2] = TriangleMeshDesc.Indices[TriIdx].v2;
	}

	// Store vertex positions
	VertPositions.Empty();
	VertPositions.AddUninitialized(NumVerts);
	for (int32 VertIdx = 0; VertIdx < TriangleMeshDesc.Vertices.Num(); VertIdx++)
	{
		VertPositions[VertIdx] = FVector3d(TriangleMeshDesc.Vertices[VertIdx]);
	}

	// Copy UV channels (checking they are correct size)
	for (int32 UVIndex = 0; UVIndex < TriangleMeshDesc.UVs.Num(); UVIndex++)
	{
		if (TriangleMeshDesc.UVs[UVIndex].Num() == NumVerts)
		{
			VertUVs.Add(TriangleMeshDesc.UVs[UVIndex]);
		}
		else
		{
			break;
		}
	}
}

void UBodySetup::AddCollisionFrom(class UBodySetup* FromSetup)
{
	AddCollisionFrom(FromSetup->AggGeom);
}

bool IsRuntime(const UBodySetup* BS)
{
    UObject* OwningObject = BS->GetOuter();
	UWorld* World = OwningObject ? OwningObject->GetWorld() : nullptr;
	return World && World->IsGameWorld();
}

DECLARE_CYCLE_STAT(TEXT("Create Physics Meshes"), STAT_CreatePhysicsMeshes, STATGROUP_Physics);

FByteBulkData* UBodySetup::GetCookedFormatData()
{
	// Find or create cooked physics data
	static FName PhysicsFormatName(FPlatformProperties::GetPhysicsFormat());
	return GetCookedData(PhysicsFormatName);
}

void UBodySetup::CreatePhysicsMeshes()
{
	// Create meshes from cooked data if not already done
	if (bCreatedPhysicsMeshes)
	{
		return;
	}

	// If we don't have any convex/trimesh data we can skip this whole function
	if (bNeverNeedsCookedCollisionData)
	{
		return;
	}

	SCOPE_CYCLE_COUNTER(STAT_CreatePhysicsMeshes);
	
	bool bClearMeshes = true;
	bool bSkipProcessFormatData = false;

	// If the data already has been deserialized from the async loading thread, use that.
	if (ChaosDerivedDataReader.IsValid())
	{
		bClearMeshes = !ProcessFormatData_Chaos(*ChaosDerivedDataReader.Get());
		ChaosDerivedDataReader.Reset();
		bSkipProcessFormatData = true;
	}

	if (!bSkipProcessFormatData)
	{
		if (FByteBulkData* FormatData = GetCookedFormatData())
		{
			bClearMeshes = !ProcessFormatData_Chaos(FormatData);
		}
		else
		{
			if (IsRuntime(this))
			{
				bClearMeshes = !RuntimeCookPhysics_Chaos();
			}
		}
	}

	// fix up invalid transform to use identity
	// this can be here because BodySetup isn't blueprintable
	if ( GetLinkerUEVersion() < VER_UE4_FIXUP_BODYSETUP_INVALID_CONVEX_TRANSFORM )
	{
		for (int32 i=0; i<AggGeom.ConvexElems.Num(); ++i)
		{
			if ( AggGeom.ConvexElems[i].GetTransform().IsValid() == false )
			{
				AggGeom.ConvexElems[i].SetTransform(FTransform::Identity);
			}
		}
	}

	// For drawing of convex elements we require an index buffer, previously we could
	// get this from a PxConvexMesh but Chaos doesn't maintain that data. Instead now
	// it is a part of the element rather than the physics geometry, if we load in an
	// element without that data present, generate a convex hull from the convex vert
	// data and extract the index data from there.
	for(FKConvexElem& Convex : AggGeom.ConvexElems)
	{
		Convex.ComputeChaosConvexIndices();
	}

	if(bClearMeshes)
	{
		ClearPhysicsMeshes();
	}
	
	bCreatedPhysicsMeshes = true;

}

void UBodySetup::CreatePhysicsMeshesAsync(FOnAsyncPhysicsCookFinished OnAsyncPhysicsCookFinished)
{
	check(IsInGameThread());

	// Don't start another cook cycle if one's already in progress
	check(CurrentCookHelper == nullptr);

	FAsyncCookHelper* NewCookHelper = new FAsyncCookHelper(this);
	if(NewCookHelper->HasWork())
	{
		FSimpleDelegateGraphTask::CreateAndDispatchWhenReady(FSimpleDelegateGraphTask::FDelegate::CreateRaw(NewCookHelper, &FAsyncCookHelper::CookAsync,
															 /*FinishDelegate=*/FSimpleDelegateGraphTask::FDelegate::CreateUObject(this, &UBodySetup::FinishCreatePhysicsMeshesAsync, NewCookHelper, OnAsyncPhysicsCookFinished)),
															 GET_STATID(STAT_PhysXCooking), nullptr, ENamedThreads::AnyThread);
		CurrentCookHelper = NewCookHelper;
	}
	else
	{
		delete NewCookHelper;
		FinishCreatePhysicsMeshesAsync(nullptr, OnAsyncPhysicsCookFinished);
	}
}

void UBodySetup::AbortPhysicsMeshAsyncCreation()
{
	check(IsInGameThread());

	// If we have a current cook helper we will ask it to cancel, null our ref to it, and ignore any results it produces.
	// Note that CreatePhysicsMeshesAsync, AbortPhysicsMeshAsyncCreation and FinishCreatePhysicsMeshesAsync all run on the game thread.
	// After this function returns we could safely call CreatePhysicsMeshesAsync again on this body from the game thread.
	if (CurrentCookHelper)
	{
		CurrentCookHelper->CancelCookAsync();
		CurrentCookHelper = nullptr;
	}
}

void UBodySetup::FinishCreatePhysicsMeshesAsync(FAsyncCookHelper* AsyncPhysicsCookHelper, FOnAsyncPhysicsCookFinished OnAsyncPhysicsCookFinished)
{
	check(IsInGameThread());

	// If a canceled CookHelper finishes delete it.
	if (AsyncPhysicsCookHelper && AsyncPhysicsCookHelper->WasCanceled())
	{
		// CurrentCookHelper could be null or it could be a new FAsyncCookHelper, but it should never be a canceled cook helper.
		check(CurrentCookHelper != AsyncPhysicsCookHelper);
		delete AsyncPhysicsCookHelper;
		return;
	}

	// Un-canceled cooks should match the CurrentCookHelper.
	// Then clear it
	check(CurrentCookHelper == AsyncPhysicsCookHelper);
	CurrentCookHelper = nullptr;

	bool bSuccess = AsyncPhysicsCookHelper != nullptr;

	if(AsyncPhysicsCookHelper)
	{
		FinishCreatingPhysicsMeshes_Chaos(*AsyncPhysicsCookHelper);

		delete AsyncPhysicsCookHelper;
	}
	else
	{
		ClearPhysicsMeshes();
		bCreatedPhysicsMeshes = true;
	}

	OnAsyncPhysicsCookFinished.ExecuteIfBound(bSuccess);
}

bool UBodySetup::ProcessFormatData_Chaos(FChaosDerivedDataReader<float, 3>& Reader)
{
	FinishCreatingPhysicsMeshes_Chaos(Reader);

	return true;
}

bool UBodySetup::ProcessFormatData_Chaos(FByteBulkData* FormatData)
{
	if(FormatData->IsLocked())
	{
		// seems it's being already processed
		return false;
	}

	FChaosDerivedDataReader<float, 3> Reader(FormatData);
	return ProcessFormatData_Chaos(Reader);
}

bool UBodySetup::RuntimeCookPhysics_Chaos()
{
	Chaos::FCookHelper Cooker(this);
	Cooker.Cook();
	FinishCreatingPhysicsMeshes_Chaos(Cooker);

	return true;
	
}

void UBodySetup::FinishCreatingPhysicsMeshes_Chaos(FChaosDerivedDataReader<float, 3>& InReader)
{
	FinishCreatingPhysicsMeshes_Chaos(InReader.ConvexGeometries, InReader.TriMeshGeometries, InReader.UVInfo, InReader.FaceRemap);
}

void UBodySetup::FinishCreatingPhysicsMeshes_Chaos(Chaos::FCookHelper& InHelper)
{
	TArray<Chaos::FConvexPtr> SimpleImplicits;
	TArray<Chaos::FTriangleMeshImplicitObjectPtr> ComplexImplicits;

	// The cooker will prepare unique implicits, body setup requires shared implicits, we do the conversion / promotion to shared
	// here and then the contents are moved into the body setup storage as part of FinishCreatingPhysicsMeshes
	for(Chaos::FImplicitObjectPtr& Simple : InHelper.SimpleImplicits)
	{
		SimpleImplicits.Add( Chaos::FConvexPtr(Simple->GetObject<Chaos::FConvex>()));
	}

	for(Chaos::FTriangleMeshImplicitObjectPtr& Complex : InHelper.ComplexImplicits)
	{
		ComplexImplicits.Add( Chaos::FTriangleMeshImplicitObjectPtr(Complex->GetObject<Chaos::FTriangleMeshImplicitObject>()));
	}

	InHelper.SimpleImplicits.Reset();
	InHelper.ComplexImplicits.Reset();

	FinishCreatingPhysicsMeshes_Chaos(SimpleImplicits, ComplexImplicits, InHelper.UVInfo, InHelper.FaceRemap);
}

void UBodySetup::FinishCreatingPhysicsMeshes_Chaos(TArray<Chaos::FConvexPtr>& ConvexImplicits, TArray<Chaos::FTriangleMeshImplicitObjectPtr>& TrimeshImplicits, FBodySetupUVInfo& InUvInfo, TArray<int32>& InFaceRemap)
{
	ClearPhysicsMeshes();

	const FString FullName = GetFullName();
	if(GetCollisionTraceFlag() != CTF_UseComplexAsSimple)
	{
		for(int32 ElementIndex = 0; ElementIndex < AggGeom.ConvexElems.Num(); ElementIndex++)
		{
			FKConvexElem& ConvexElem = AggGeom.ConvexElems[ElementIndex];

			if(CHAOS_ENSURE((ElementIndex < ConvexImplicits.Num())
			   && ConvexImplicits[ElementIndex]->IsValidGeometry()))
			{
				// Optimization: Only update internal convex data because we just deserialized the convex and the FKConvexElem convex information matches the chaos implicit one
				ConvexElem.SetConvexMeshObject(MoveTemp(ConvexImplicits[ElementIndex]), FKConvexElem::EConvexDataUpdateMethod::UpdateConvexDataOnlyIfMissing);

#if TRACK_CHAOS_GEOMETRY
				ConvexElem.GetChaosConvexMesh()->Track(Chaos::MakeSerializable(ConvexElem.GetChaosConvexMesh()), FullName);
#endif

				if(ConvexElem.GetChaosConvexMesh()->IsPerformanceWarning())
				{
					const FString& PerformanceString = ConvexElem.GetChaosConvexMesh()->PerformanceWarningAndSimplifaction();
					CHAOS_LOG(LogPhysics, Warning, TEXT("TConvex Name:%s, Element [%d], %s"), *FullName, ElementIndex, *PerformanceString);
				}
			}
			else
			{
				if(ElementIndex >= ConvexImplicits.Num())
				{
					CHAOS_LOG(LogPhysics, Warning, TEXT("InReader.ConvexImplicitObjects.Num() [%d], AggGeom.ConvexElems.Num() [%d]"),
						   ConvexImplicits.Num(), AggGeom.ConvexElems.Num());
				}
				UE_LOGF(LogPhysics, Warning, "TConvex Name:%ls, Element [%d] has no Geometry", *FullName, ElementIndex);
			}
		}
		ConvexImplicits.Reset();
	}

	TriMeshGeometries = MoveTemp(TrimeshImplicits);
	UVInfo = MoveTemp(InUvInfo);
	FaceRemap = MoveTemp(InFaceRemap);
#if TRACK_CHAOS_GEOMETRY
	for(auto& TriMesh : TriMeshGeometries)
	{
		TriMesh->Track(Chaos::MakeSerializable(TriMesh), FullName);
	}
#endif

	// Force trimesh collisions off
	for(auto& TriMesh : TriMeshGeometries)
	{
		TriMesh->SetDoCollide(false);
	}

	// Clear the cooked data
	if(!GIsEditor && !bSharedCookedData)
	{
		CookedFormatData.FlushData();
	}

	bCreatedPhysicsMeshes = true;
}


void UBodySetup::ClearPhysicsMeshes()
{
	for (int32 i = 0; i < AggGeom.ConvexElems.Num(); i++)
	{
		FKConvexElem* ConvexElem = &(AggGeom.ConvexElems[i]);
		ConvexElem->ResetChaosConvexMesh();
	}
	TriMeshGeometries.Reset();

	bCreatedPhysicsMeshes = false;

	// Also clear render info
	AggGeom.FreeRenderInfo();
}

DECLARE_CYCLE_STAT(TEXT("AddShapesToRigidActor"), STAT_AddShapesToActor, STATGROUP_Physics);
DECLARE_CYCLE_STAT(TEXT("AddGeomToSolver"), STAT_AddGeomToSolver, STATGROUP_Physics);

void UBodySetup::AddShapesToRigidActor_AssumesLocked(
	FBodyInstance* OwningInstance, 
	FVector& Scale3D, 
	UPhysicalMaterial* SimpleMaterial,
	TArray<UPhysicalMaterial*>& ComplexMaterials,
	TArray<FPhysicalMaterialMaskParams>& ComplexMaterialMasks,
	const FBodyCollisionData& BodyCollisionData,
	const FTransform& RelativeTM, 
	TArray<FPhysicsShapeHandle>* NewShapes)
{
	SCOPE_CYCLE_COUNTER(STAT_AddShapesToActor);

	check(OwningInstance);

	// in editor, there are a lot of things relying on body setup to create physics meshes
	CreatePhysicsMeshes();

	// To AddGeometry in interface
	// if almost zero, set min scale
	// @todo fixme
	if (Scale3D.IsNearlyZero())
	{
		// set min scale
		Scale3D = FVector(0.1f);
	}

	FGeometryAddParams AddParams;
	AddParams.bDoubleSided = bDoubleSidedGeometry;
	AddParams.CollisionData = BodyCollisionData;
	AddParams.CollisionTraceType = GetCollisionTraceFlag();
	AddParams.Scale = Scale3D;
	AddParams.SimpleMaterial = SimpleMaterial;
	AddParams.ComplexMaterials = TArrayView<UPhysicalMaterial*>(ComplexMaterials);
	AddParams.ComplexMaterialMasks = TArrayView<FPhysicalMaterialMaskParams>(ComplexMaterialMasks);
	AddParams.LocalTransform = RelativeTM;
	AddParams.WorldTransform = OwningInstance->GetUnrealWorldTransform();
	AddParams.Geometry = &AggGeom;
	AddParams.TriMeshGeometries = MakeArrayView(TriMeshGeometries);

	{
		SCOPE_CYCLE_COUNTER(STAT_AddGeomToSolver);
		FPhysicsInterface::AddGeometry(OwningInstance->GetPhysicsActor(), AddParams, NewShapes);
	}
}

void UBodySetup::RemoveSimpleCollision()
{
	InvalidatePhysicsData();
	AggGeom.EmptyElements();
}

void UBodySetup::RescaleSimpleCollision( FVector BuildScale )
{
	if( FVector(BuildScale3D) != BuildScale )
	{					
		// Back out the old scale when applying the new scale
		const FVector ScaleMultiplier3D = (BuildScale / FVector(BuildScale3D));

		for (int32 i = 0; i < AggGeom.ConvexElems.Num(); i++)
		{
			FKConvexElem* ConvexElem = &(AggGeom.ConvexElems[i]);

			FTransform ConvexTrans = ConvexElem->GetTransform();
			FVector ConvexLoc = ConvexTrans.GetLocation();
			ConvexLoc *= ScaleMultiplier3D;
			ConvexTrans.SetLocation(ConvexLoc);
			ConvexElem->SetTransform(ConvexTrans);

			TArray<FVector>& Vertices = ConvexElem->VertexData;
			for (int32 VertIndex = 0; VertIndex < Vertices.Num(); ++VertIndex)
			{
				Vertices[VertIndex] *= ScaleMultiplier3D;
			}

			ConvexElem->UpdateElemBox();
		}

		// @todo Deal with non-vector properties by just applying the max value for the time being
		const float ScaleMultiplier = ScaleMultiplier3D.GetMax();

		for (int32 i = 0; i < AggGeom.SphereElems.Num(); i++)
		{
			FKSphereElem* SphereElem = &(AggGeom.SphereElems[i]);

			SphereElem->Center *= ScaleMultiplier3D;
			SphereElem->Radius *= ScaleMultiplier;
		}

		for (int32 i = 0; i < AggGeom.BoxElems.Num(); i++)
		{
			FKBoxElem* BoxElem = &(AggGeom.BoxElems[i]);

			BoxElem->Center *= ScaleMultiplier3D;
			BoxElem->X *= ScaleMultiplier3D.X;
			BoxElem->Y *= ScaleMultiplier3D.Y;
			BoxElem->Z *= ScaleMultiplier3D.Z;
		}

		for (int32 i = 0; i < AggGeom.SphylElems.Num(); i++)
		{
			FKSphylElem* SphylElem = &(AggGeom.SphylElems[i]);

			SphylElem->Center *= ScaleMultiplier3D;
			SphylElem->Radius *= ScaleMultiplier;
			SphylElem->Length *= ScaleMultiplier;
		}

		BuildScale3D = BuildScale;
	}
}

void UBodySetup::InvalidatePhysicsData()
{
	ClearPhysicsMeshes();
	// We use the BodySetupGuid as a change indicator for DDC. It would be better to create it deterministically
	// from the physics data but that is not yet implemented. As a workaround we assign it an arbitrary unique value
	// every time the physics data changes.
	BodySetupGuid = FGuid::NewGuid();
	if (!bSharedCookedData)
	{
		CookedFormatData.FlushData();
	}
}

void UBodySetup::BeginDestroy()
{
	Super::BeginDestroy();

	AggGeom.FreeRenderInfo();
}	

void UBodySetup::FinishDestroy()
{
	ClearPhysicsMeshes();
	Super::FinishDestroy();
}




void UBodySetup::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);
	Ar.UsingCustomVersion(FExternalPhysicsCustomObjectVersion::GUID);
	Ar.UsingCustomVersion(FFortniteMainBranchObjectVersion::GUID);
	Ar.UsingCustomVersion(FFortniteReleaseBranchCustomObjectVersion::GUID);

	// Load GUID (or create one for older versions)
	Ar << BodySetupGuid;

	// If we loaded a ZERO Guid, fix that
	if (Ar.IsLoading() && !BodySetupGuid.IsValid())
	{
		MarkPackageDirty();
		FString PathName = GetPathName();
		UE_LOGF(LogPhysics, Log, "FIX GUID FOR: %ls", *PathName);
		// For the autocreated guid for an old version, we construct it to a constant value; this is important
		// for determinism during cooking if a legacy BodySetup is encountered during cooking. The BodySetupGuid is used
		// as a change-indicator for the BodySetup's DDC key, so this constant value would ordinarily be a problem, but
		// it is okay in this legacy-version handling because the physics data stored in the package will always be the
		// same, unless the package is resaved. And when the package is resaved, the new code will calculate a unique
		// BodySetupGuid for it and store that BodySetupGuid in the serialized data.
		BodySetupGuid = FGuid::NewDeterministicGuid(PathName);
	}

	bool bCooked = Ar.IsCooking();
	Ar << bCooked;

	bool bDuplicating = (Ar.GetPortFlags() & PPF_Duplicate) != 0;

	if (bCooked)
	{
#if WITH_EDITOR
		if (Ar.IsCooking())
		{
			// Make sure to reset bHasCookedCollision data to true before calling GetCookedData for cooking
			bHasCookedCollisionData = true;
			FName Format = Ar.CookingTarget()->GetPhysicsFormat(this);
			bHasCookedCollisionData = GetCookedData(Format) != NULL; // Get the data from the DDC or build it

			TArray<FName> ActualFormatsToSave;
			ActualFormatsToSave.Add(Format);

			FArchive_Serialize_BitfieldBool(Ar, bHasCookedCollisionData);
			
			CookedFormatData.Serialize(Ar, this, &ActualFormatsToSave, !bSharedCookedData);

#if VERIFY_COOKED_PHYS_DATA
			// Verify that the cooked data matches the uncooked data
			if(GetCollisionTraceFlag() != CTF_UseComplexAsSimple)
			{
				UObject* Outer = GetOuter();

				for(TPair<FName, FByteBulkData*>& TestFormat : UseCookedFormatData->Formats)
				{
					FByteBulkData* BulkData = TestFormat.Value;
					if(BulkData && BulkData->GetBulkDataSize() > 0)
					{
						FPhysXCookingDataReader PhysDataReader(*BulkData, &UVInfo);

						if(PhysDataReader.ConvexMeshes.Num() != AggGeom.ConvexElems.Num() || PhysDataReader.TriMeshes.Num() != TriMeshes.Num())
						{
							// Cooked data doesn't match our current geo
							UE_LOGF(LogPhysics, Warning, "Body setup cooked data for component %ls does not match uncooked geo. Convex: %d, %d, Trimesh: %d, %d", Outer ? *Outer->GetName() : TEXT("None"), AggGeom.ConvexElems.Num(), PhysDataReader.ConvexMeshes.Num(), TriMeshes.Num(), PhysDataReader.TriMeshes.Num());
						}
					}
				}
			}
#endif
		}
		else
#endif
		{
			if (Ar.UEVer() >= VER_UE4_STORE_HASCOOKEDDATA_FOR_BODYSETUP)
			{
				// CL#14327190 Removed cooked implicit collision structures from the UBodySetup.
				// UBodySetups saved with support for cooked implicit geometry store a counter for the number 
				// of implicit objects saved to the file. This count needs to be removed from the input stream. 
				// Note: We only need to extract the count, not the array. Editor operations for populating the 
				// implicit array were never added, so it's expected that the actual implicit array would 
				// always be empty.
				bool bTemp = bHasCookedCollisionData;
				Ar << bTemp;
				bHasCookedCollisionData = bTemp;
			}
			CookedFormatData.Serialize(Ar, this);
		}
	}

#if WITH_EDITOR
	AggGeom.FixupDeprecated( Ar );
#endif

#if WITH_EDITOR

	if (Ar.IsLoading())
	{
		const bool bForceIndexRebuild = Ar.CustomVer(FExternalPhysicsCustomObjectVersion::GUID) < FExternalPhysicsCustomObjectVersion::ForceRebuildBodySetupIndices;
		for (FKConvexElem& Convex : AggGeom.ConvexElems)
		{
			// Reset potentially corrupted index data to correctly rebuild below
			if (bForceIndexRebuild)
			{
				Convex.IndexData.Reset();
			}
			// Build an index buffer if we don't have one, either as a consequence of the check above or loading in a mesh that has never been
			// processed with Chaos previously
			Convex.ComputeChaosConvexIndices();
		}
	}
#endif


	// Levelset Serialization support for BodySetup.
	if (Ar.CustomVer(FFortniteMainBranchObjectVersion::GUID) >= FFortniteMainBranchObjectVersion::LevelsetSerializationSupportForBodySetup
		&& Ar.CustomVer(FFortniteReleaseBranchCustomObjectVersion::GUID) < FFortniteReleaseBranchCustomObjectVersion::DisableLevelset_v14_10)
	{
		using namespace Chaos;

		TArray<Chaos::FImplicitObjectPtr> ChaosImplicitObjects;
		FChaosArchive ChaosAr(Ar);

		int32 NumImplicits = 0;
		
		if (Ar.IsLoading())
		{
			ChaosImplicitObjects.Reset();

			Ar << NumImplicits;

			for (int i = 0; i < NumImplicits; i++)
			{
				if (FImplicitObject* ImplicitObject = FImplicitObject::SerializationFactory(ChaosAr, nullptr))
				{
					ImplicitObject->Serialize(Ar);
					ChaosImplicitObjects.Add(Chaos::FImplicitObjectPtr(ImplicitObject));
				}
			}
		}
		/*
		else if (Ar.IsSaving())
		{
			for (int i = 0; i < ChaosImplicitObjects.Num(); i++)
			{
				if (ChaosImplicitObjects[i])
				{
					NumImplicits++;
				}
			}
			Ar << NumImplicits;

			for (int i = 0; i < ChaosImplicitObjects.Num(); i++)
			{
				if (ChaosImplicitObjects[i])
				{
					FImplicitObject::SerializationFactory(ChaosAr, ChaosImplicitObjects[i].Get());
					ChaosImplicitObjects[i]->Serialize(Ar);
				}
			}
		}
		*/
	}

	if (bCooked && Ar.IsLoading())
	{
		// Deserialize bulk inline data inside the serialize function to benefit from async loading thread when possible.
		if (FByteBulkData* FormatData = GetCookedFormatData())
		{
			ChaosDerivedDataReader = MakeUnique<FChaosDerivedDataReader<float, 3>>(FormatData);
		}
	}
}

void UBodySetup::PostLoad()
{
	LLM_SCOPE(ELLMTag::Physics);
	Super::PostLoad();

	// Our owner needs to be post-loaded before us else they may not have loaded
	// their data yet.
	UObject* Outer = GetOuter();
	if (Outer)
	{
		Outer->ConditionalPostLoad();
	}

#if WITH_EDITORONLY_DATA
	if ( GetLinkerUEVersion() < VER_UE4_BUILD_SCALE_VECTOR )
	{
		BuildScale3D = FVector(BuildScale_DEPRECATED);
	}
#endif

	DefaultInstance.FixupData(this);

	if ( GetLinkerUEVersion() < VER_UE4_REFACTOR_PHYSICS_BLENDING )
	{
		if ( bAlwaysFullAnimWeight_DEPRECATED )
		{
			PhysicsType = PhysType_Simulated;
		}
		else if ( DefaultInstance.bSimulatePhysics == false )
		{
			PhysicsType = PhysType_Kinematic;
		}
		else
		{
			PhysicsType = PhysType_Default;
		}
	}

	if ( GetLinkerUEVersion() < VER_UE4_BODYSETUP_COLLISION_CONVERSION )
	{
		if ( DefaultInstance.GetCollisionEnabled() == ECollisionEnabled::NoCollision )
		{
			CollisionReponse = EBodyCollisionResponse::BodyCollision_Disabled;
		}
	}

#if WITH_EDITOR
	// UStaticMesh::PostLoad will always call CreatePhysicsMeshes()
	// on its BodySetup in its own postload. We don't need to unnecessarily
	// stall on async staticmesh compilation to perform it here.
	UStaticMesh* StaticMesh = Cast<UStaticMesh>(GetOuter());
	if (StaticMesh && StaticMesh->IsCompiling())
	{
		return;
	}
#endif

	// Compress to whatever formats the active target platforms want
	ITargetPlatformManagerModule* TPM = GetTargetPlatformManager();
	if (TPM)
	{
		const TArray<ITargetPlatform*>& Platforms = TPM->GetActiveTargetPlatforms();

		for (int32 Index = 0; Index < Platforms.Num(); Index++)
		{
			GetCookedData(Platforms[Index]->GetPhysicsFormat(this));
		}
	}

	// If Deferring physics creation, skip so we can call CreatePhysicsMeshes in parallel.
	if (GEnableDeferredPhysicsCreation == false)
	{
		CreatePhysicsMeshes();
	}
}

void UBodySetup::UpdateTriMeshVertices(const TArray<FVector> & NewPositions)
{
	SCOPE_CYCLE_COUNTER(STAT_UpdateTriMeshVertices);

	ensure(false);
}

template <bool bPositionAndNormal, bool bUseConvexShapes>
float GetClosestPointAndNormalImpl(const UBodySetup* BodySetup, const FVector& WorldPosition, const FTransform& LocalToWorld, FVector* ClosestWorldPosition, FVector* FeatureNormal)
{
	float ClosestDist = FLT_MAX;
	FVector TmpPosition, TmpNormal;
	int32 NumShapeTested = 0;

	//Note that this function is optimized for BodySetup with few elements. This is more common. If we want to optimize the case with many elements we should really return the element during the distance check to avoid pointless iteration
	for (const FKSphereElem& SphereElem : BodySetup->AggGeom.SphereElems)
	{
		NumShapeTested++;
		if constexpr (bPositionAndNormal)
		{
			const float Dist = SphereElem.GetClosestPointAndNormal(WorldPosition, LocalToWorld, TmpPosition, TmpNormal);

			if(Dist < ClosestDist)
			{
				*ClosestWorldPosition = TmpPosition;
				*FeatureNormal = TmpNormal;
				ClosestDist = Dist;
			}
		}
		else
		{
			const float Dist = SphereElem.GetShortestDistanceToPoint(WorldPosition, LocalToWorld);
			ClosestDist = Dist < ClosestDist ? Dist : ClosestDist;
		}
	}

	for (const FKSphylElem& SphylElem : BodySetup->AggGeom.SphylElems)
	{
		NumShapeTested++;
		if constexpr (bPositionAndNormal)
		{
			const float Dist = SphylElem.GetClosestPointAndNormal(WorldPosition, LocalToWorld, TmpPosition, TmpNormal);

			if (Dist < ClosestDist)
			{
				*ClosestWorldPosition = TmpPosition;
				*FeatureNormal = TmpNormal;
				ClosestDist = Dist;
			}
		}
		else
		{
			const float Dist = SphylElem.GetShortestDistanceToPoint(WorldPosition, LocalToWorld);
			ClosestDist = Dist < ClosestDist ? Dist : ClosestDist;
		}
	}

	for (const FKBoxElem& BoxElem : BodySetup->AggGeom.BoxElems)
	{
		NumShapeTested++;
		if constexpr (bPositionAndNormal)
		{
			const float Dist = BoxElem.GetClosestPointAndNormal(WorldPosition, LocalToWorld, TmpPosition, TmpNormal);

			if (Dist < ClosestDist)
			{
				*ClosestWorldPosition = TmpPosition;
				*FeatureNormal = TmpNormal;
				ClosestDist = Dist;
			}
		}
		else
		{
			const float Dist =  BoxElem.GetShortestDistanceToPoint(WorldPosition, LocalToWorld);
			ClosestDist = Dist < ClosestDist ? Dist : ClosestDist;
		}
	}

	if constexpr (bUseConvexShapes)
	{
		NumShapeTested++;
		for (const FKConvexElem& ConvexElem : BodySetup->AggGeom.ConvexElems)
		{
			if constexpr (bPositionAndNormal)
			{
				const float Dist = ConvexElem.GetClosestPointAndNormal(WorldPosition, LocalToWorld, TmpPosition, TmpNormal);

				if (Dist < ClosestDist)
				{
					*ClosestWorldPosition = TmpPosition;
					*FeatureNormal = TmpNormal;
					ClosestDist = Dist;
				}
			}
			else
			{
				const float Dist = ConvexElem.GetShortestDistanceToPoint(WorldPosition, LocalToWorld);
				ClosestDist = Dist < ClosestDist ? Dist : ClosestDist;
			}
		}
	}

	if (NumShapeTested > 0 && ClosestDist == FLT_MAX)
	{
		UE_LOGF(LogPhysics, Warning, "GetClosestPointAndNormalImpl ClosestDist for BodySetup %ls is coming back as FLT_MAX. WorldPosition = %ls, LocalToWorld = %ls", *BodySetup->GetFullName(), *WorldPosition.ToString(), *LocalToWorld.ToHumanReadableString());
	}

	return ClosestDist;
}

float UBodySetup::GetShortestDistanceToPoint(const FVector& WorldPosition, const FTransform& LocalToWorld, bool bUseConvexShapes) const
{
	if (bUseConvexShapes)
	{
		return GetClosestPointAndNormalImpl<false, true>(this, WorldPosition, LocalToWorld, nullptr, nullptr);
	}
	else
	{
		return GetClosestPointAndNormalImpl<false, false>(this, WorldPosition, LocalToWorld, nullptr, nullptr);
	}
}

float UBodySetup::GetClosestPointAndNormal(const FVector& WorldPosition, const FTransform& LocalToWorld, FVector& ClosestWorldPosition, FVector& FeatureNormal, bool bUseConvexShapes) const
{
	if (bUseConvexShapes)
	{
		return GetClosestPointAndNormalImpl<true, true>(this, WorldPosition, LocalToWorld, &ClosestWorldPosition, &FeatureNormal);
	}
	else
	{
		return GetClosestPointAndNormalImpl<true, false>(this, WorldPosition, LocalToWorld, &ClosestWorldPosition, &FeatureNormal);
	}
}

#if WITH_EDITOR
void UBodySetup::BeginCacheForCookedPlatformData(const ITargetPlatform* TargetPlatform)
{
}

bool UBodySetup::IsCachedCookedPlatformDataLoaded(const ITargetPlatform* TargetPlatform)
{
	if (IInterface_CollisionDataProvider* CDP = Cast<IInterface_CollisionDataProvider>(GetOuter()))
	{
		bool bInUseAllTriData = true;
		if (!CDP->PollAsyncPhysicsTriMeshData(bInUseAllTriData))
		{
			return false;
		}
	}
	FBodySetupTryCookResult CookResult = TryGetCookedData(TargetPlatform->GetPhysicsFormat(this));

	if (CookResult.CookResult == EBodySetupCookResult::FailedToLock)
	{
		UE_LOGF(LogPhysics, Warning, "Multiple writes to BodySetup bulk data detected during cook for setup: %ls", *GetPathName());
		return false;
	}
	if (CookResult.CookResult == EBodySetupCookResult::DataFormatProviderUnavailable)
	{
		UE_LOGF(LogPhysics, Error,
			"This BodySetup is on a skeletal component, and is supposed to copy its FormatContainer from the body setup of its SkeletalMesh asset. But that Body setup is no longer available. %ls",
			*GetPathName());
		// We cannot recover from this, so report that we are done trying to load, to avoid softlocking the cook.
		return true;
	}

	return true;
}

void UBodySetup::ClearCachedCookedPlatformData( const ITargetPlatform* TargetPlatform )
{

}
#endif

bool UBodySetup::CalcUVAtLocation(const FVector& BodySpaceLocation, int32 FaceIndex, int32 UVChannel, FVector2D& UV) const
{
	bool bSuccess = false;

	if (UVInfo.VertUVs.IsValidIndex(UVChannel) && UVInfo.IndexBuffer.IsValidIndex(FaceIndex * 3 + 2))
	{
		int32 Index0 = UVInfo.IndexBuffer[FaceIndex * 3 + 0];
		int32 Index1 = UVInfo.IndexBuffer[FaceIndex * 3 + 1];
		int32 Index2 = UVInfo.IndexBuffer[FaceIndex * 3 + 2];

		FVector Pos0 = UVInfo.VertPositions[Index0];
		FVector Pos1 = UVInfo.VertPositions[Index1];
		FVector Pos2 = UVInfo.VertPositions[Index2];

		FVector2D UV0 = UVInfo.VertUVs[UVChannel][Index0];
		FVector2D UV1 = UVInfo.VertUVs[UVChannel][Index1];
		FVector2D UV2 = UVInfo.VertUVs[UVChannel][Index2];

		// Transform hit location from world to local space.
		// Find barycentric coords
		FVector BaryCoords = FMath::ComputeBaryCentric2D(BodySpaceLocation, Pos0, Pos1, Pos2);
		// Use to blend UVs
		UV = (BaryCoords.X * UV0) + (BaryCoords.Y * UV1) + (BaryCoords.Z * UV2);

		bSuccess = true;
	}

	return bSuccess;
}

#if WITH_EDITOR

bool ShouldSkipDDC(UBodySetup* InSetup, FString& OutReason)
{
	// Building body setup is so fast and is invalidated so often because of guids usage
	// that its preferable to avoid using DDC queries unless we have to process something significant.

	const int32 SkipDDCThreshold = CVarBodySetupSkipDDCThreshold.GetValueOnAnyThread();
	if (SkipDDCThreshold > 0)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(ShouldSkipDDC);

		// Some numbers on AMD TR 3970X single-thread
		//    18603 verts 6976 indices takes 13 ms to build
		//     9793 verts 5878 indices takes  9 ms to build
		//    Cloud DDC access is generally between 20 ms and 150 ms
		//    Local DDC access is generally between 0.02 ms and 5 ms on SSD

		// By using the DDC2 API, we could be more specific about how many
		// vertices should represent enough work to warrant a cloud query
		// versus allowing local queries only.

		if (InSetup->GetCollisionTraceFlag() != CTF_UseComplexAsSimple && InSetup->AggGeom.ConvexElems.Num() > 0)
		{
			int32 VerticeCount = 0;
			for (const FKConvexElem& ConvexElem : InSetup->AggGeom.ConvexElems)
			{
				VerticeCount += ConvexElem.VertexData.Num();
			}

			if (VerticeCount >= SkipDDCThreshold)
			{
				OutReason = FString::Printf(TEXT("AggGeom Vertice Count %d"), VerticeCount);
				return false;
			}
		}

		if (InSetup->GetCollisionTraceFlag() != CTF_UseSimpleAsComplex)
		{
			UObject* CDPObj = InSetup->GetOuter();
			IInterface_CollisionDataProvider* CDP = Cast<IInterface_CollisionDataProvider>(CDPObj);

			if (CDP && CDP->ContainsPhysicsTriMeshData(InSetup->bMeshCollideAll))
			{
				FTriMeshCollisionDataEstimates Estimates;
				if (CDP->GetTriMeshSizeEstimates(Estimates, InSetup->bMeshCollideAll) && Estimates.VerticeCount >= SkipDDCThreshold)
				{
					OutReason = FString::Printf(TEXT("CDP Vertice Count %lld"), Estimates.VerticeCount);
					return false;
				}
			}
		}

		// Defaults to true
		return true;
	}
	else
	{
		return false;
	}
}

UE_TRACE_EVENT_BEGIN(Cpu, BodySetupDDCFetch, NoSync)
	UE_TRACE_EVENT_FIELD(UE::Trace::WideString, FetchReason)
UE_TRACE_EVENT_END()

UBodySetup::EBodySetupCookResult TryGetDDCBuiltData(FByteBulkData* OutResult, TUniquePtr<FChaosDerivedDataCooker>&& InBuilder, UBodySetup* InSetup, bool bInIsRuntime)
{
	TArray<uint8> OutData;
	check(InBuilder);

	if (InBuilder->CanBuild())
	{
		COOK_STAT(FCookStats::FScopedStatsCounter Timer = BodySetupCookStats::UsageStats.TimeSyncWork());

		// Debugging switch, force builder to always run
		bool bSkipDDC = false;

		bool bDataWasBuilt = false;
		bool bDDCHit = false;

		FString FetchReason;
		if(!bSkipDDC && !ShouldSkipDDC(InSetup, FetchReason))
		{
#if CPUPROFILERTRACE_ENABLED
			UE_TRACE_LOG_SCOPED_T(Cpu, BodySetupDDCFetch, CpuChannel)
				<< BodySetupDDCFetch.FetchReason(*FetchReason);
#endif
			// GetSynchronous takes ownership of and will delete the ReleasedBuilder
			FChaosDerivedDataCooker* ReleasedBuilder = InBuilder.Release();
			bDDCHit = GetDerivedDataCacheRef().GetSynchronous(ReleasedBuilder, OutData, &bDataWasBuilt);

			// Only compute hit/miss if DDC was not skipped
			COOK_STAT(Timer.AddHitOrMiss(!bDDCHit || bDataWasBuilt ? FCookStats::CallStats::EHitOrMiss::Miss : FCookStats::CallStats::EHitOrMiss::Hit, OutData.Num()));
		}
		else
		{
			bDataWasBuilt = true;
			InBuilder->Build(OutData);
			COOK_STAT(Timer.TrackCyclesOnly());
		}
	}

	if (OutData.Num())
	{
		if(OutResult->IsLocked())
		{
			return UBodySetup::EBodySetupCookResult::FailedToLock;
		}

		OutResult->Lock(LOCK_READ_WRITE);
		FMemory::Memcpy(OutResult->Realloc(OutData.Num()), OutData.GetData(), OutData.Num());
		OutResult->Unlock();
	}
	else if(!bInIsRuntime)	//only want to warn if DDC cooking failed - if it's really trying to use runtime and we can't, the runtime cooker code will catch it
	{
		UE_LOGF(LogPhysics, Warning, "Attempt to build physics data for %ls when we are unable to.", *InSetup->GetPathName());
		return UBodySetup::EBodySetupCookResult::FailedToCook;
	}

	return UBodySetup::EBodySetupCookResult::NoError;
}

#endif //#if WITH_EDITOR

FByteBulkData* UBodySetup::GetCookedData(FName Format)
{
	UBodySetup::FBodySetupTryCookResult CookResult = TryGetCookedData(Format);

	// Failing to cook means we'll have no physics data which is not ideal but works and the cook will have warned the user.
	// Failing to lock means we have multiple paths attempting to write into the bulk data buffer which is more serious. Separating these
	// case to help trace any race conditions arising during the physics cook
	check(CookResult.CookResult != EBodySetupCookResult::FailedToLock);

	return CookResult.Data;
}

UBodySetup::FBodySetupTryCookResult UBodySetup::TryGetCookedData(FName Format)
{
	EBodySetupCookResult CookDataResult = EBodySetupCookResult::NoError;

	if(IsTemplate())
	{
		return { nullptr, CookDataResult };
	}

	// Geometry should never have collision data, cooked data will never be present
	if(bNeverNeedsCookedCollisionData)
	{
		return { nullptr, CookDataResult };
	}

	IInterface_CollisionDataProvider* CDP = Cast<IInterface_CollisionDataProvider>(GetOuter());

	// If there is nothing to cook or if we are reading data from a cooked package for an asset with no collision, 
	// we want to return here
	if((AggGeom.ConvexElems.Num() == 0 && CDP == nullptr) || !bHasCookedCollisionData)
	{
		return { nullptr, CookDataResult };
	}

	FFormatContainer* UsedFormatData = &CookedFormatData;
	if (bHasCookedFormatProvider)
	{
		UsedFormatData = nullptr;
		const USkeletalMesh* SkeletalMeshWithFormat = CookedFormatProvider.Get();
		if (SkeletalMeshWithFormat)
		{
			UBodySetup* BodySetupWithFormat = SkeletalMeshWithFormat->GetBodySetup();
			if (BodySetupWithFormat)
			{
				UsedFormatData = &BodySetupWithFormat->CookedFormatData;
			}
		}
		if (!UsedFormatData)
		{
			return { nullptr, EBodySetupCookResult::DataFormatProviderUnavailable };
		}
	}

	FByteBulkData* Result = nullptr;
	const bool bIsRuntime = IsRuntime(this);

#if WITH_EDITOR
	bool bFormatDataContainsFormat;
	{
		FReadScopeLock ScopeLock(CookedFormatDataMutex);
		bFormatDataContainsFormat = UsedFormatData->Contains(Format);
		if (bFormatDataContainsFormat)
		{
			Result = &UsedFormatData->GetFormat(Format);
		}
	}
	if (!bFormatDataContainsFormat)
	{
		SCOPE_CYCLE_COUNTER(STAT_PhysXCooking);

		// Note: Check ContainsPhysicsTriMeshData before looking at the number of convex elems, to ensure the side effects of ContainsPhysicsTriMeshData happen
		// (specifically, for static mesh this will ensure the mesh render data is already built)
		// Note: We also do this *before* the below scope lock, to avoid deadlock 
		// (because ContainsPhysicsTriMeshData can call GetRenderData() for static mesh, which will wait for async build to run, 
		//  which will try to take this same lock on a separate thread when the async mesh build calls this same method)
		if ((CDP == nullptr || CDP->ContainsPhysicsTriMeshData(bMeshCollideAll) == false) && AggGeom.ConvexElems.Num() == 0)
		{
			return { nullptr, CookDataResult };
		}

		// Create the builder and gather the data from our StaticMesh to build the DDC key.
		// Gathering data from StaticMesh will block on the StaticMesh's compile if it is in progress, so do that
		// gather outside of our lock.
		// We do not want a FGCObject to be created to prevent garbage collection of our own UBodySetup*
		// because the scope and lifetime is well defined and FGCObject can't be created on other threads
		// during garbage collection, which would prevent this function from being run asynchronously.
		const bool bUseRefHolder = false;
		TUniquePtr<FChaosDerivedDataCooker> PhysicsDerivedCooker( new FChaosDerivedDataCooker(this, Format, bUseRefHolder));
		PhysicsDerivedCooker->GatherKeyData();

		// Now that we have waited for any static mesh compile to complete (or we are already on the static mesh compile
		// thread), attempt to enter the write lock that we use to guard access to the physics data. Below calling GetFormat and
		// TryGetDDCBuiltData will both mutate shared state.
		FWriteScopeLock ScopeLock(CookedFormatDataMutex);

		// Check the lock condition again, we only want the first thread in here to actually build the data,
		// while it's building other threads should wait and proceed after the build completes.
		bFormatDataContainsFormat = UsedFormatData->Contains(Format);
		if (bFormatDataContainsFormat)
		{
			// We were a waiting thread while another thread built the data, just take it from the container now
			Result = &UsedFormatData->GetFormat(Format);
		}
		else
		{
			Result = &UsedFormatData->GetFormat(Format);
			CookDataResult = TryGetDDCBuiltData(Result, MoveTemp(PhysicsDerivedCooker), this, bIsRuntime);
		}
	}
#else // #if WITH_EDITOR
	{
		Result = &UsedFormatData->GetFormat(Format);
	}
#endif // #else !WITH_EDITOR

	check(Result);

	// we don't return empty bulk data...but we save it to avoid thrashing the DDC
	Result = Result->GetBulkDataSize() > 0 ? Result : nullptr;
	return {Result, CookDataResult};
}

void UBodySetup::GetGeometryDDCKey(FString& OutString) const
{
	// If the geometry changes in a way not controlled below, increment the key version to invalidate DDC entries
	enum { BODY_SETUP_GEOMETRY_KEY_VER = 0 };

	FString MeshIdString("CDP_BODYSETUP");

	IInterface_CollisionDataProvider* CDP = Cast<IInterface_CollisionDataProvider>(GetOuter());
	if(CDP)
	{
		CDP->GetMeshId(MeshIdString);
	}

	OutString = FString::Printf(TEXT("%s_%s_%s_%d_%d_%d_%d_%f_%f_%d"),
		*BodySetupGuid.ToString(),
		*MeshIdString,
		*AggGeom.MakeDDCKey().ToString(),
		(int32)bGenerateNonMirroredCollision,
		(int32)bGenerateMirroredCollision,
		(int32)UPhysicsSettings::Get()->bSupportUVFromHitResults,
		(int32)GetCollisionTraceFlag(),
		UPhysicsSettings::Get()->SolverOptions.CollisionMarginFraction,
		UPhysicsSettings::Get()->SolverOptions.CollisionMarginMax,
		(int32)BODY_SETUP_GEOMETRY_KEY_VER);

	if (bSupportUVsAndFaceRemap)
	{
		OutString += FString("_1");
	}

	if (!bSupportFaceRemapOnMeshBVH)
	{
		OutString += FString("_F");
	}
	if (!bSupportVertexRemap)
	{
		OutString += FString("_V");
	}
}

void UBodySetup::PostInitProperties()
{
	Super::PostInitProperties();

	if (!IsTemplate())
	{
		// When constructing a BodySetup, we initialize its guid to a constant value; this is important
		// for determinism during cooking if a new BodySetup is created during cooking. The BodySetupGuid is used
		// as a change-indicator for the BodySetup's DDC key, so it is not possible to use a constant value for it
		// for changes to the BodySetup, but since a new BodySetup is created into an initial state that never varies,
		// making the initial value of BodySetupGuid a constant is okay.
		BodySetupGuid = FGuid::NewDeterministicGuid(GetPathName());
	}
}

#if WITH_EDITOR

void UBodySetup::PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent)
{
	if(PropertyChangedEvent.MemberProperty && PropertyChangedEvent.MemberProperty->GetFName() == GET_MEMBER_NAME_CHECKED(UBodySetup, AggGeom))
	{
		UStaticMesh* StaticMesh = GetTypedOuter<UStaticMesh>();
		if(StaticMesh)
		{
			for(UStaticMeshComponent* StaticMeshComponent : TObjectRange<UStaticMeshComponent>())
			{
				if(StaticMeshComponent->GetStaticMesh() == StaticMesh)
				{
					// it needs to recreate IF it already has been created
					if(StaticMeshComponent->IsPhysicsStateCreated())
					{
						StaticMeshComponent->RecreatePhysicsState();
					}
				}
			}
		}
	}
}

void UBodySetup::PostEditUndo()
{
	Super::PostEditUndo();

	// If we have any convex elems, ensure they are recreated whenever anything is modified!
	if(AggGeom.ConvexElems.Num() > 0)
	{
		InvalidatePhysicsData();
		CreatePhysicsMeshes();
	}
}

void UBodySetup::CopyBodySetupProperty(const UBodySetup* Other)
{
	BoneName = Other->BoneName;
	PhysicsType = Other->PhysicsType;
	bConsiderForBounds = Other->bConsiderForBounds;
	bMeshCollideAll = Other->bMeshCollideAll;
	bDoubleSidedGeometry = Other->bDoubleSidedGeometry;
	bGenerateNonMirroredCollision = Other->bGenerateNonMirroredCollision;
	bSharedCookedData = Other->bSharedCookedData;
	bGenerateMirroredCollision = Other->bGenerateMirroredCollision;
	PhysMaterial = Other->PhysMaterial;
	CollisionReponse = Other->CollisionReponse;
	CollisionTraceFlag = Other->CollisionTraceFlag;
	DefaultInstance = Other->DefaultInstance;
	WalkableSlopeOverride = Other->WalkableSlopeOverride;
	BuildScale3D = Other->BuildScale3D;
}

EDataValidationResult UBodySetup::IsDataValid(FDataValidationContext& Context) const
{
	EDataValidationResult Result = EDataValidationResult::Valid;

	// Check that the body has at least one shape
	int32 NumElements = AggGeom.GetElementCount();
	if (NumElements == 0)
	{
		Context.AddError(FText::Format(LOCTEXT("UBodySetupHasNoCollision", "Bone {0} requires at least one collision shape"), FText::FromName(BoneName)));
		Result = EDataValidationResult::Invalid;
	}

	// Chekc that simulated bodies have at least one shape that contributes to mass, otherwise
	// we cannot calculate the inertia, even if the mass is exlicitly set.
	// @todo(physics): should we check non-simulated bodies? The simulation type can be changed in the runtime...
	if (PhysicsType == EPhysicsType::PhysType_Simulated)
	{
		int32 NumMassContributors = 0;
		for (int32 ElementIndex = 0; ElementIndex < NumElements; ++ElementIndex)
		{
			const FKShapeElem* Shape = AggGeom.GetElement(ElementIndex);
			if (Shape->GetContributeToMass())
			{
				++NumMassContributors;
			}
		}
		
		if (NumMassContributors == 0)
		{
			Context.AddError(FText::Format(LOCTEXT("UBodySetupHasNoMass", "Bone {0} requires at least one shape with 'Contribute to Mass' set to 'true'"), FText::FromName(BoneName)));
			Result = EDataValidationResult::Invalid;
		}
	}

	return Result;
}

#endif // WITH_EDITOR

void UBodySetup::GetResourceSizeEx(FResourceSizeEx& CumulativeResourceSize)
{
	Super::GetResourceSizeEx(CumulativeResourceSize);

	// Cooked mesh data is flushed after mesh creation if not sharing and not in editor, account through rough serilization estimates.
	if (!GIsEditor && !bSharedCookedData)
	{
		if (GetCollisionTraceFlag() != CTF_UseComplexAsSimple)
		{
			for (int32 ElementIndex = 0; ElementIndex < AggGeom.ConvexElems.Num(); ElementIndex++)
			{
				FKConvexElem& ConvexElem = AggGeom.ConvexElems[ElementIndex];
				if (ConvexElem.GetChaosConvexMesh() != nullptr)
				{
					TArray<uint8> Data;
					FMemoryWriter MemAr(Data);
					Chaos::FChaosArchive ChaosAr(MemAr);
					ConvexElem.GetChaosConvexMesh()->Serialize(ChaosAr);
					CumulativeResourceSize.AddDedicatedSystemMemoryBytes(Data.Num());
				}
			}
		}

		for (auto& TriMesh : TriMeshGeometries)
		{
			if (TriMesh.GetReference() != nullptr)
			{
				TArray<uint8> Data;
				FMemoryWriter MemAr(Data);
				Chaos::FChaosArchive ChaosAr(MemAr);
				TriMesh.GetReference()->Serialize(ChaosAr);
				CumulativeResourceSize.AddDedicatedSystemMemoryBytes(Data.Num());
			}
		}
	}

	if (CookedFormatData.Contains(FPlatformProperties::GetPhysicsFormat()))
	{
		const FByteBulkData& FmtData = CookedFormatData.GetFormat(FPlatformProperties::GetPhysicsFormat());
		CumulativeResourceSize.AddDedicatedSystemMemoryBytes(FmtData.GetBulkDataSize());
	}

	CumulativeResourceSize.AddDedicatedSystemMemoryBytes(FaceRemap.GetAllocatedSize());
	
	// Count any UV info
	UVInfo.GetResourceSizeEx(CumulativeResourceSize);
}

#if WITH_EDITORONLY_DATA
void FKAggregateGeom::FixupDeprecated(FArchive& Ar)
{
	for (auto SphereElemIt = SphereElems.CreateIterator(); SphereElemIt; ++SphereElemIt)
	{
		SphereElemIt->FixupDeprecated(Ar);
	}

	for (auto BoxElemIt = BoxElems.CreateIterator(); BoxElemIt; ++BoxElemIt)
	{
		BoxElemIt->FixupDeprecated(Ar);
	}

	for (auto SphylElemIt = SphylElems.CreateIterator(); SphylElemIt; ++SphylElemIt)
	{
		SphylElemIt->FixupDeprecated(Ar);
	}
}
#endif

FVector::FReal FKAggregateGeom::GetVolume(const FVector& Scale) const
{
	return GetScaledVolume(Scale);
}

FVector::FReal FKAggregateGeom::GetScaledVolume(const FVector& Scale3D) const
{
	float Volume = 0.0f;

	VisitSimElements([&Scale3D, &Volume] <typename ElementType> (const ElementType & Elem)
	{
		Volume += Elem.GetScaledVolume(Scale3D);
	});

	return Volume;
}

FVector::FReal FKAggregateGeom::GetScaledVolumeForMassContribution(const FVector& Scale3D) const
{
	float Volume = 0.0f;

	VisitSimElements([&Scale3D, &Volume] <typename ElementType> (const ElementType& Elem)
	{
		if(Elem.GetContributeToMass())
		{
			Volume += Elem.GetScaledVolume(Scale3D);
		}
	});

	return Volume;
}

FGuid FKAggregateGeom::MakeDDCKey() const
{
	UScriptStruct* StructType = FKAggregateGeom::StaticStruct();
	TArray<uint8> Bytes;
	Bytes.Reserve(64);
	FMemoryWriter MemAr(Bytes);

	StructType->SerializeTaggedProperties(MemAr, (uint8*)this, StructType, nullptr);

	FSHA1 Sha;
	Sha.Update(Bytes.GetData(), Bytes.Num());
	Sha.Final();

	uint32 Hash[5];
	Sha.GetHash((uint8*)Hash);
	FGuid OutGuid(Hash[0] ^ Hash[4], Hash[1], Hash[2], Hash[3]);

	return OutGuid;
}

int32 FKAggregateGeom::GetElementCount(EAggCollisionShape::Type Type) const
{
	switch (Type)
	{
	case EAggCollisionShape::Box:
		return BoxElems.Num();
	case EAggCollisionShape::Convex:
		return ConvexElems.Num();
	case EAggCollisionShape::Sphyl:
		return SphylElems.Num();
	case EAggCollisionShape::Sphere:
		return SphereElems.Num();
	case EAggCollisionShape::TaperedCapsule:
		return TaperedCapsuleElems.Num();
	case EAggCollisionShape::LevelSet:
		return LevelSetElems.Num();
	default:
		return 0;
	}
}


FKConvexElem::FKConvexElem()
	: FKShapeElem(EAggCollisionShape::Convex)
	, ElemBox(ForceInit)
	, Transform(FTransform::Identity)
{}

FKConvexElem::FKConvexElem(const FKConvexElem& Other)
{
	CloneElem(Other);
}

FKConvexElem::~FKConvexElem()
{

}

const FKConvexElem& FKConvexElem::operator=(const FKConvexElem& Other)
{
	ensureMsgf(!ChaosConvex, TEXT("We are leaking memory. Why are we calling the assignment operator on an element that has already allocated resources?"));
	ResetChaosConvexMesh();
	CloneElem(Other);
	return *this;
}

/** Helper function to safely copy instances of this shape*/
void FKConvexElem::CloneElem(const FKConvexElem& Other)
{
	Super::CloneElem(Other);
	VertexData = Other.VertexData;
	IndexData = Other.IndexData;
	ElemBox = Other.ElemBox;
	Transform = Other.Transform;
}

void FKConvexElem::ScaleElem(FVector DeltaSize, float MinSize)
{
	FTransform ScaledTransform = GetTransform();
	ScaledTransform.SetScale3D(ScaledTransform.GetScale3D() + DeltaSize);
	SetTransform(ScaledTransform);
}

// References: 
// http://amp.ece.cmu.edu/Publication/Cha/icip01_Cha.pdf
// http://stackoverflow.com/questions/1406029/how-to-calculate-the-volume-of-a-3d-mesh-object-the-surface-of-which-is-made-up
float SignedVolumeOfTriangle(const FVector3f& p1, const FVector3f& p2, const FVector3f& p3) 
{
	return FVector3f::DotProduct(p1, FVector3f::CrossProduct(p2, p3)) / 6.0f;
}

FVector::FReal FKConvexElem::GetVolume(const FVector& Scale) const
{
	return GetScaledVolume(Scale);
}

FVector::FReal FKConvexElem::GetScaledVolume(const FVector& Scale) const
{
	float Volume = 0.0f;

	if (ChaosConvex != nullptr)
	{
		Volume = FMath::Abs(Scale.X * Scale.Y * Scale.Z) * ChaosConvex->GetVolume();
	}

	return Volume;
}

void FKConvexElem::SetConvexMeshObject(Chaos::FConvexPtr&& InChaosConvex, EConvexDataUpdateMethod ConvexDataUpdateMethod /* = EConvexDataUpdateMethod::AlwaysUpdateConvexData */)
{
	ChaosConvex = MoveTemp(InChaosConvex);
	
	const bool bForceCompute = (ConvexDataUpdateMethod == EConvexDataUpdateMethod::AlwaysUpdateConvexData);
	ComputeChaosConvexIndices(bForceCompute);
}

void FKConvexElem::ResetChaosConvexMesh()
{
	ChaosConvex.SafeRelease();
}

ENGINE_API void FKConvexElem::ComputeChaosConvexIndices(bool bForceCompute)
{
	// these indices are not needed for simulation, but are also used by NavMesh
	if (bForceCompute || IndexData.Num() == 0)
	{
		IndexData = GetChaosConvexIndices();
	}
}

TArray<int32> FKConvexElem::GetChaosConvexIndices() const
{
	TArray<int32> ResultIndexData;
	const int32 NumVerts = VertexData.Num();
	if (NumVerts > 0)
	{
		TArray<Chaos::FConvex::FVec3Type> ConvexVertices;
		ConvexVertices.SetNum(NumVerts);

		for (int32 VertIndex = 0; VertIndex < NumVerts; ++VertIndex)
		{
			ConvexVertices[VertIndex] = VertexData[VertIndex];
		}

		Chaos::FConvexBuilder::BuildIndices(ConvexVertices, ResultIndexData);
	}

	return ResultIndexData;
}

#if WITH_EDITORONLY_DATA
void FKSphereElem::FixupDeprecated( FArchive& Ar )
{
	if ( Ar.IsLoading() && Ar.UEVer() < VER_UE4_REFACTOR_PHYSICS_TRANSFORMS )
	{
		Center = TM_DEPRECATED.GetOrigin();
	}
}
#endif

float FKSphereElem::GetShortestDistanceToPoint(const FVector& WorldPosition, const FTransform& LocalToWorldTM) const
{
	FKSphereElem ScaledSphere = GetFinalScaled(LocalToWorldTM.GetScale3D(), FTransform::Identity);

	const FVector Dir = LocalToWorldTM.TransformPositionNoScale(ScaledSphere.Center) - WorldPosition;
	const float DistToCenter = Dir.Size();
	const float DistToEdge = DistToCenter - ScaledSphere.Radius;
	
	return DistToEdge > UE_SMALL_NUMBER ? DistToEdge : 0.f;
}

float FKSphereElem::GetClosestPointAndNormal(const FVector& WorldPosition, const FTransform& LocalToWorldTM, FVector& ClosestWorldPosition, FVector& Normal) const
{
	FKSphereElem ScaledSphere = GetFinalScaled(LocalToWorldTM.GetScale3D(), FTransform::Identity);

	const FVector Dir = LocalToWorldTM.TransformPositionNoScale(ScaledSphere.Center) - WorldPosition;
	const float DistToCenter = Dir.Size();
	const float DistToEdge = FMath::Max(DistToCenter - ScaledSphere.Radius, 0.f);

	if(DistToCenter > UE_SMALL_NUMBER)
	{
		Normal = -Dir.GetUnsafeNormal();
	}
	else
	{
		Normal = FVector::ZeroVector;
	}
	
	ClosestWorldPosition = WorldPosition - Normal*DistToEdge;

	return DistToEdge;
}

void FKSphereElem::ScaleElem(FVector DeltaSize, float MinSize)
{
	// Find element with largest magnitude, btu preserve sign.
	float DeltaRadius = DeltaSize.X;

	if (FMath::Abs(DeltaSize.Y) > FMath::Abs(DeltaRadius))
	{
		DeltaRadius = DeltaSize.Y;
	}

	if (FMath::Abs(DeltaSize.Z) > FMath::Abs(DeltaRadius))
	{
		DeltaRadius = DeltaSize.Z;
	}

	Radius = FMath::Max(Radius + DeltaRadius, MinSize);
}

FKSphereElem FKSphereElem::GetFinalScaled(const FVector& Scale3D, const FTransform& RelativeTM) const
{
	float MinScale, MinScaleAbs;
	FVector Scale3DAbs;

	SetupNonUniformHelper(Scale3D * RelativeTM.GetScale3D(), MinScale, MinScaleAbs, Scale3DAbs);

	FKSphereElem ScaledSphere = *this;
	ScaledSphere.Radius *= MinScaleAbs;

	ScaledSphere.Center = RelativeTM.TransformPosition(Center) * Scale3D;


	return ScaledSphere;
}

#if WITH_EDITORONLY_DATA
void FKBoxElem::FixupDeprecated( FArchive& Ar )
{
	if ( Ar.IsLoading() && Ar.UEVer() < VER_UE4_REFACTOR_PHYSICS_TRANSFORMS )
	{
		Center = TM_DEPRECATED.GetOrigin();
		Orientation_DEPRECATED = TM_DEPRECATED.ToQuat();
	}

	Ar.UsingCustomVersion(FAnimPhysObjectVersion::GUID);
	if (Ar.IsLoading() && Ar.CustomVer(FAnimPhysObjectVersion::GUID) < FAnimPhysObjectVersion::BoxSphylElemsUseRotators)
	{
		Rotation = Orientation_DEPRECATED.Rotator();
	}
}
#endif

void FKBoxElem::ScaleElem(FVector DeltaSize, float MinSize)
{
	// Sizes are lengths, so we double the delta to get similar increase in size.
	X = FMath::Max<FVector::FReal>(X + 2 * DeltaSize.X, MinSize);
	Y = FMath::Max<FVector::FReal>(Y + 2 * DeltaSize.Y, MinSize);
	Z = FMath::Max<FVector::FReal>(Z + 2 * DeltaSize.Z, MinSize);
}


FKBoxElem FKBoxElem::GetFinalScaled(const FVector& Scale3D, const FTransform& RelativeTM) const
{
	float MinScale, MinScaleAbs;
	FVector Scale3DAbs;

	SetupNonUniformHelper(Scale3D * RelativeTM.GetScale3D(), MinScale, MinScaleAbs, Scale3DAbs);

	FKBoxElem ScaledBox = *this;
	ScaledBox.X *= Scale3DAbs.X;
	ScaledBox.Y *= Scale3DAbs.Y;
	ScaledBox.Z *= Scale3DAbs.Z;

	FTransform ScaleTransform(FQuat::Identity, FVector::ZeroVector, Scale3D); 
	FTransform BoxTransform = GetTransform() * RelativeTM * ScaleTransform;
	ScaledBox.SetTransform(BoxTransform);

	return ScaledBox;
}

float FKBoxElem::GetShortestDistanceToPoint(const FVector& WorldPosition, const FTransform& BoneToWorldTM) const
{
	const FKBoxElem& ScaledBox = GetFinalScaled(BoneToWorldTM.GetScale3D(), FTransform::Identity);
	const FTransform LocalToWorldTM = GetTransform() * BoneToWorldTM;
	const FVector LocalPosition = LocalToWorldTM.InverseTransformPositionNoScale(WorldPosition);
	const FVector LocalPositionAbs = LocalPosition.GetAbs();

	const FVector HalfPoint(ScaledBox.X*0.5f, ScaledBox.Y*0.5f, ScaledBox.Z*0.5f);
	const FVector Delta = LocalPositionAbs - HalfPoint;
	const FVector Errors = FVector(FMath::Max<FVector::FReal>(Delta.X, 0), FMath::Max<FVector::FReal>(Delta.Y, 0), FMath::Max<FVector::FReal>(Delta.Z, 0));
	const float Error = Errors.Size();

	return Error > UE_SMALL_NUMBER ? Error : 0.f;
}

float FKBoxElem::GetClosestPointAndNormal(const FVector& WorldPosition, const FTransform& BoneToWorldTM, FVector& ClosestWorldPosition, FVector& Normal) const
{
	const FKBoxElem& ScaledBox = GetFinalScaled(BoneToWorldTM.GetScale3D(), FTransform::Identity);
	const FTransform LocalToWorldTM = GetTransform() * BoneToWorldTM;
	const FVector LocalPosition = LocalToWorldTM.InverseTransformPositionNoScale(WorldPosition);

	const float HalfX = ScaledBox.X * 0.5f;
	const float HalfY = ScaledBox.Y * 0.5f;
	const float HalfZ = ScaledBox.Z * 0.5f;
	
	const FVector ClosestLocalPosition(FMath::Clamp<FVector::FReal>(LocalPosition.X, -HalfX, HalfX), FMath::Clamp<FVector::FReal>(LocalPosition.Y, -HalfY, HalfY), FMath::Clamp<double>(LocalPosition.Z, -HalfZ, HalfZ));
	ClosestWorldPosition = LocalToWorldTM.TransformPositionNoScale(ClosestLocalPosition);

	const FVector LocalDelta = LocalPosition - ClosestLocalPosition;
	float Error = LocalDelta.Size();
	
	bool bIsOutside = Error > UE_SMALL_NUMBER;
	
	const FVector LocalNormal = bIsOutside ? LocalDelta.GetUnsafeNormal() : FVector::ZeroVector;

	ClosestWorldPosition = LocalToWorldTM.TransformPositionNoScale(ClosestLocalPosition);
	Normal = LocalToWorldTM.TransformVectorNoScale(LocalNormal);
	
	return bIsOutside ? Error : 0.f;
}

#if WITH_EDITORONLY_DATA
void FKSphylElem::FixupDeprecated( FArchive& Ar )
{
	if ( Ar.IsLoading() && Ar.UEVer() < VER_UE4_REFACTOR_PHYSICS_TRANSFORMS )
	{
		Center = TM_DEPRECATED.GetOrigin();
		Orientation_DEPRECATED = TM_DEPRECATED.ToQuat();
	}

	Ar.UsingCustomVersion(FAnimPhysObjectVersion::GUID);
	if (Ar.IsLoading() && Ar.CustomVer(FAnimPhysObjectVersion::GUID) < FAnimPhysObjectVersion::BoxSphylElemsUseRotators)
	{
		Rotation = Orientation_DEPRECATED.Rotator();
	}
}
#endif

void FKSphylElem::ScaleElem(FVector DeltaSize, float MinSize)
{
	float DeltaRadius = DeltaSize.X;
	if (FMath::Abs(DeltaSize.Y) > FMath::Abs(DeltaRadius))
	{
		DeltaRadius = DeltaSize.Y;
	}

	float DeltaHeight = DeltaSize.Z;
	float radius = FMath::Max(Radius + DeltaRadius, MinSize);
	float length = Length + DeltaHeight;

	length += Radius - radius;
	length = FMath::Max(0.f, length);

	Radius = radius;
	Length = length;
}

FKSphylElem FKSphylElem::GetFinalScaled(const FVector& Scale3D, const FTransform& RelativeTM) const
{
	FKSphylElem ScaledSphylElem = *this;

	float MinScale, MinScaleAbs;
	FVector Scale3DAbs;

	SetupNonUniformHelper(Scale3D * RelativeTM.GetScale3D(), MinScale, MinScaleAbs, Scale3DAbs);

	ScaledSphylElem.Radius = GetScaledRadius(Scale3DAbs);
	ScaledSphylElem.Length = GetScaledCylinderLength(Scale3DAbs);

	const FTransform ScaleTransform(FQuat::Identity, FVector::ZeroVector, Scale3D);
	const FTransform RotationTransform(ScaledSphylElem.Rotation, FVector::ZeroVector, Scale3D);
	const FTransform ScaledRotationTransform = RotationTransform * ScaleTransform;
	
	const FVector LocalOrigin = RelativeTM.TransformPosition(Center) * Scale3D;
	ScaledSphylElem.Center = LocalOrigin;
	ScaledSphylElem.Rotation = FRotator(RelativeTM.GetRotation() * ScaledRotationTransform.GetRotation());

	return ScaledSphylElem;
}

float FKSphylElem::GetScaledRadius(const FVector& Scale3D) const
{
	const FVector Scale3DAbs = Scale3D.GetAbs();
	const float RadiusScale = FMath::Max(Scale3DAbs.X, Scale3DAbs.Y);
	return FMath::Clamp(Radius * RadiusScale, 0.1f, GetScaledHalfLength(Scale3DAbs));
}

float FKSphylElem::GetScaledCylinderLength(const FVector& Scale3D) const
{
	return FMath::Max<float>(0.1f, (GetScaledHalfLength(Scale3D) - GetScaledRadius(Scale3D)) * 2.f);
}

float FKSphylElem::GetScaledHalfLength(const FVector& Scale3D) const
{
	return FMath::Max<float>((Length + Radius * 2.0f) * FMath::Abs(Scale3D.Z) * 0.5f, 0.1f);
}

float FKSphylElem::GetShortestDistanceToPoint(const FVector& WorldPosition, const FTransform& BoneToWorldTM) const
{
	const FKSphylElem ScaledSphyl = GetFinalScaled(BoneToWorldTM.GetScale3D(), FTransform::Identity);

	const FTransform LocalToWorldTM = GetTransform() * BoneToWorldTM;
	const FVector ErrorScale = LocalToWorldTM.GetScale3D();
	const FVector LocalPosition = LocalToWorldTM.InverseTransformPositionNoScale(WorldPosition);
	const FVector LocalPositionAbs = LocalPosition.GetAbs();
	
	
	const FVector Target(LocalPositionAbs.X, LocalPositionAbs.Y, FMath::Max<FVector::FReal>(LocalPositionAbs.Z - ScaledSphyl.Length * 0.5f, 0.f));	//If we are above half length find closest point to cap, otherwise to cylinder
	const float Error = FMath::Max(Target.Size() - ScaledSphyl.Radius, 0.f);

	return Error > UE_SMALL_NUMBER ? Error : 0.f;
}

float FKSphylElem::GetClosestPointAndNormal(const FVector& WorldPosition, const FTransform& BoneToWorldTM, FVector& ClosestWorldPosition, FVector& Normal) const
{
	const FKSphylElem ScaledSphyl = GetFinalScaled(BoneToWorldTM.GetScale3D(), FTransform::Identity);

	const FTransform LocalToWorldTM = GetTransform() * BoneToWorldTM;
	const FVector ErrorScale = LocalToWorldTM.GetScale3D();
	const FVector LocalPosition = LocalToWorldTM.InverseTransformPositionNoScale(WorldPosition);
	
	const float HalfLength = 0.5f * ScaledSphyl.Length;
	const float TargetZ = FMath::Clamp<FVector::FReal>(LocalPosition.Z, -HalfLength, HalfLength);	//We want to move to a sphere somewhere along the capsule axis

	const FVector WorldSphere = LocalToWorldTM.TransformPositionNoScale(FVector(0.f, 0.f, TargetZ));
	const FVector Dir = WorldSphere - WorldPosition;
	const float DistToCenter = Dir.Size();
	const float DistToEdge = FMath::Max(DistToCenter - ScaledSphyl.Radius, 0.f);

	bool bIsOutside = DistToCenter > UE_SMALL_NUMBER;
	if (bIsOutside)
	{
		Normal = -Dir.GetUnsafeNormal();
	}
	else
	{
		Normal = FVector::ZeroVector;
	}

	ClosestWorldPosition = WorldPosition - Normal*DistToEdge;

	return bIsOutside ? DistToEdge : 0.f;
}

void FKTaperedCapsuleElem::ScaleElem(FVector DeltaSize, float MinSize)
{
	float DeltaRadius0 = DeltaSize.X;
	float DeltaRadius1 = DeltaSize.Y;

	float DeltaHeight = DeltaSize.Z;
	float NewRadius0 = FMath::Max(Radius0 + DeltaRadius0, MinSize);
	float NewRadius1 = FMath::Max(Radius1 + DeltaRadius1, MinSize);
	float NewLength = Length + DeltaHeight;

	NewLength += ((Radius1 - NewRadius1) + (Radius0 - NewRadius0)) * 0.5f;
	NewLength = FMath::Max(0.f, NewLength);

	Radius0 = NewRadius0;
	Radius1 = NewRadius1;
	Length = NewLength;
}

FKTaperedCapsuleElem FKTaperedCapsuleElem::GetFinalScaled(const FVector& Scale3D, const FTransform& RelativeTM) const
{
	FKTaperedCapsuleElem ScaledTaperedCapsuleElem = *this;

	float MinScale, MinScaleAbs;
	FVector Scale3DAbs;

	SetupNonUniformHelper(Scale3D * RelativeTM.GetScale3D(), MinScale, MinScaleAbs, Scale3DAbs);

	GetScaledRadii(Scale3DAbs, ScaledTaperedCapsuleElem.Radius0, ScaledTaperedCapsuleElem.Radius1);
	ScaledTaperedCapsuleElem.Length = GetScaledCylinderLength(Scale3DAbs);
	ScaledTaperedCapsuleElem.Width *= Scale3DAbs.X;

	const FTransform ScaleTransform(FQuat::Identity, FVector::ZeroVector, Scale3D);
	const FTransform RotationTransform(ScaledTaperedCapsuleElem.Rotation, FVector::ZeroVector, Scale3D);
	const FTransform ScaledRotationTransform = RotationTransform * ScaleTransform;
	
	const FVector LocalOrigin = RelativeTM.TransformPosition(Center) * Scale3D;
	ScaledTaperedCapsuleElem.Center = LocalOrigin;
	ScaledTaperedCapsuleElem.Rotation = FRotator(RelativeTM.GetRotation() * ScaledRotationTransform.GetRotation());

	return ScaledTaperedCapsuleElem;
}

void FKTaperedCapsuleElem::GetScaledRadii(const FVector& Scale3D, float& OutRadius0, float& OutRadius1) const
{
	const FVector Scale3DAbs = Scale3D.GetAbs();
	const float RadiusScale = FMath::Max(Scale3DAbs.X, Scale3DAbs.Y);
	OutRadius0 = FMath::Clamp(Radius0 * RadiusScale, 0.1f, GetScaledHalfLength(Scale3DAbs));
	OutRadius1 = FMath::Clamp(Radius1 * RadiusScale, 0.1f, GetScaledHalfLength(Scale3DAbs));
}

float FKTaperedCapsuleElem::GetScaledCylinderLength(const FVector& Scale3D) const
{
	float ScaledRadius0, ScaledRadius1;
	GetScaledRadii(Scale3D, ScaledRadius0, ScaledRadius1);
	return FMath::Max(0.1f, ((GetScaledHalfLength(Scale3D) * 2.0f) - (ScaledRadius0 + ScaledRadius1)));
}

float FKTaperedCapsuleElem::GetScaledHalfLength(const FVector& Scale3D) const
{
	return FMath::Max<float>((Length + Radius0 + Radius1) * FMath::Abs(Scale3D.Z) * 0.5f, 0.1f);
}

void FKTaperedCapsuleElem::GetScaledExtrudedCylinderLengths(const FVector& Scale3D, float& OutLength0, float& OutLength1) const
{
	float ScaledRadius0, ScaledRadius1;
	GetScaledRadii(Scale3D, ScaledRadius0, ScaledRadius1);
	float ScaledExtendedHalfLength0, ScaledExtendedHalfLength1;
	GetScaledExtrudedHalfLengths(Scale3D, ScaledExtendedHalfLength0, ScaledExtendedHalfLength1);
	OutLength0 = FMath::Max(0.1f, ((ScaledExtendedHalfLength0 - ScaledRadius0) * 2.0f));
	OutLength1 = FMath::Max(0.1f, ((ScaledExtendedHalfLength1 - ScaledRadius1) * 2.0f));
}

void FKTaperedCapsuleElem::GetScaledExtrudedHalfLengths(const FVector& Scale3D, float& OutLength0, float& OutLength1) const
{
	OutLength0 = FMath::Max<float>((Width + 2.f * Radius0) * FMath::Abs(Scale3D.X) * 0.5f, 0.1f);
	OutLength1 = FMath::Max<float>((Width + 2.f * Radius1) * FMath::Abs(Scale3D.X) * 0.5f, 0.1f);
}

float FKConvexElem::GetClosestPointAndNormal(const FVector& WorldPosition, const FTransform& BoneToWorldTM, FVector& ClosestWorldPosition, FVector& Normal) const
{
	float MinScale, MinScaleAbs;
	FVector Scale3DAbs;
	SetupNonUniformHelper(BoneToWorldTM.GetScale3D() * GetTransform().GetScale3D(), MinScale, MinScaleAbs, Scale3DAbs);

	const FTransform LocalToWorldTM = GetTransform() * BoneToWorldTM;
	const FVector LocalPosition = LocalToWorldTM.InverseTransformPositionNoScale(WorldPosition);
	if (ChaosConvex)
	{
		Chaos::FVec3 OutNormal;
		Chaos::FReal Phi = ChaosConvex->PhiWithNormalScaled(LocalPosition, Scale3DAbs, OutNormal);
		Normal = LocalToWorldTM.TransformVectorNoScale(OutNormal);
		ClosestWorldPosition = WorldPosition - Normal * Phi;
		return Phi > UE_SMALL_NUMBER ? Phi : 0.f;
	}
	return 0.f;
}

float FKConvexElem::GetShortestDistanceToPoint(const FVector& WorldPosition, const FTransform& BoneToWorldTM) const
{
	float MinScale, MinScaleAbs;
	FVector Scale3DAbs;
	SetupNonUniformHelper(BoneToWorldTM.GetScale3D() * GetTransform().GetScale3D(), MinScale, MinScaleAbs, Scale3DAbs);

	const FTransform LocalToWorldTM = GetTransform() * BoneToWorldTM;
	const FVector LocalPosition = LocalToWorldTM.InverseTransformPositionNoScale(WorldPosition);
	if (ChaosConvex)
	{
		Chaos::FVec3 OutNormal;
		Chaos::FReal Phi = ChaosConvex->PhiWithNormalScaled(LocalPosition, Scale3DAbs, OutNormal);
		return Phi > UE_SMALL_NUMBER ? Phi : 0.f;
	}
	return 0.f;
}

class UPhysicalMaterial* UBodySetup::GetPhysMaterial() const
{
	UPhysicalMaterial* PhysMat = PhysMaterial;

	if (PhysMat == NULL && GEngine != NULL)
	{
		PhysMat = GEngine->DefaultPhysMaterial;
	}
	return PhysMat;
}

float UBodySetup::CalculateMass(const UPrimitiveComponent* Component) const
{
	FVector ComponentScale(1.0f, 1.0f, 1.0f);
	const FBodyInstance* BodyInstance = &DefaultInstance;
	float MassScale = DefaultInstance.MassScale;

	const UPrimitiveComponent* OuterComp = Component != NULL ? Component : Cast<UPrimitiveComponent>(GetOuter());
	if (OuterComp)
	{
		ComponentScale = OuterComp->GetComponentScale();

		BodyInstance = &OuterComp->BodyInstance;

		const USkinnedMeshComponent* SkinnedMeshComp = Cast<const USkinnedMeshComponent>(OuterComp);
		if (SkinnedMeshComp != NULL)
		{
			const FBodyInstance* Body = SkinnedMeshComp->GetBodyInstance(BoneName);

			if (Body != NULL)
			{
				BodyInstance = Body;
			}
		}
	}

	if(BodyInstance->bOverrideMass)
	{
		return BodyInstance->GetMassOverride();
	}

	UPhysicalMaterial* PhysMat = BodyInstance->GetSimplePhysicalMaterial();
	MassScale = BodyInstance->MassScale;

	// physical material - nothing can weigh less than hydrogen (0.09 kg/m^3)
	float DensityKGPerCubicUU = 1.0f;
	float RaiseMassToPower = 0.75f;
	if (PhysMat)
	{
		DensityKGPerCubicUU = FMath::Max(0.00009f, PhysMat->Density * 0.001f);
		RaiseMassToPower = PhysMat->RaiseMassToPower;
	}

	// Then scale mass to avoid big differences between big and small objects.
	const FVector::FReal BasicVolume = GetScaledVolume(ComponentScale, EVolumeCalculationMethod::OnlyMassRelevantGeoms);

	// The below TODO is probably fixed now. GetSCaledVolume handles negative scales...
	//@TODO: Some static meshes are triggering this - disabling until content can be analyzed - ensureMsgf(BasicVolume >= 0.0f, TEXT("UBodySetup::CalculateMass(%s) - The volume of the aggregate geometry is negative"), *Component->GetReadableName());

	const FVector::FReal BasicMass = FMath::Max(BasicVolume, FVector::FReal(0)) * DensityKGPerCubicUU;

	const FVector::FReal UsePow = FMath::Clamp(RaiseMassToPower, FVector::FReal(UE_KINDA_SMALL_NUMBER), FVector::FReal(1));
	const FVector::FReal RealMass = FMath::Pow(BasicMass, UsePow);

	return float(RealMass * MassScale);
}

float UBodySetup::GetVolume(const FVector& Scale) const
{
	return GetScaledVolume(Scale);
}

FVector::FReal UBodySetup::GetScaledVolume(const FVector& Scale, EVolumeCalculationMethod InMethod) const
{
	if(InMethod == EVolumeCalculationMethod::AllGeoms)
	{
		return AggGeom.GetScaledVolume(Scale);
	}

	return AggGeom.GetScaledVolumeForMassContribution(Scale);
}

/** Helper function to safely copy instances of this shape*/
void FKLevelSetElem::CloneElem(const FKLevelSetElem& Other)
{
	Super::CloneElem(Other);
	LevelSet = Other.LevelSet;
	Transform = Other.Transform;
}

void FKLevelSetElem::ScaleElem(FVector DeltaSize, float MinSize)
{
	FTransform ScaledTransform = GetTransform();
	ScaledTransform.SetScale3D(ScaledTransform.GetScale3D() + DeltaSize);
	SetTransform(ScaledTransform);
}


/** Helper function to safely copy instances of this shape*/
void FKSkinnedLevelSetElem::CloneElem(const FKSkinnedLevelSetElem& Other)
{
	Super::CloneElem(Other);
	WeightedLatticeLevelSet = Other.WeightedLatticeLevelSet;
}

void FKMLLevelSetElem::CloneElem(const FKMLLevelSetElem& Other)
{
	Super::CloneElem(Other);
	MLLevelSet = Other.MLLevelSet;
	NNESignedDistanceModelData = Other.NNESignedDistanceModelData;
	NNEIncorrectZoneModelData = Other.NNEIncorrectZoneModelData;
}

void FKSkinnedTriangleMeshElem::CloneElem(const FKSkinnedTriangleMeshElem& Other)
{
	Super::CloneElem(Other);
	SkinnedTriangleMesh = Other.SkinnedTriangleMesh;
}

#if WITH_EDITOR

bool FBodySetupObjectTextFactory::CanCreateClass(UClass* InObjectClass, bool& bOmitSubObjs) const
{
	return (InObjectClass->IsChildOf<UBodySetup>());
}

void FBodySetupObjectTextFactory::ProcessConstructedObject(UObject* NewObject)
{
	check(NewObject);
	if (NewObject->IsA<UBodySetup>())
	{
		NewBodySetups.Add(Cast<UBodySetup>(NewObject));
	}
}

#endif // WITH_EDITOR

#undef LOCTEXT_NAMESPACE

