// Copyright Epic Games, Inc. All Rights Reserved.
#include "PhysicsEngine/BodyUtils.h"

#include "PhysicsEngine/BodyInstance.h"
#include "PhysicalMaterials/PhysicalMaterial.h"

#include "Chaos/MassProperties.h"
#include "Chaos/Rotation.h"
#include "Chaos/Utilities.h"
#include "Physics/Experimental/ChaosInterfaceUtils.h"

namespace BodyUtils
{
	namespace CVars
	{
		bool bPhysicsComNudgeAdjustInertia = true;
		FAutoConsoleVariableRef CVarPhysicsComNudgeAffectsInertia(TEXT("p.ComNudgeAffectsInertia"), bPhysicsComNudgeAdjustInertia, TEXT(""));
	}


	inline float KgPerM3ToKgPerCm3(float KgPerM3)
	{
		//1m = 100cm => 1m^3 = (100cm)^3 = 1000000cm^3
		//kg/m^3 = kg/1000000cm^3
		const float M3ToCm3Inv = 1.f / (100.f * 100.f * 100.f);
		return KgPerM3 * M3ToCm3Inv;
	}

	inline float gPerCm3ToKgPerCm3(float gPerCm3)
	{
		//1000g = 1kg
		//kg/cm^3 = 1000g/cm^3 => g/cm^3 = kg/1000 cm^3
		const float gToKG = 1.f / 1000.f;
		return gPerCm3 * gToKG;
	}

	inline float GetBodyInstanceDensity(const FBodyInstance* OwningBodyInstance)
	{
		// physical material - nothing can weigh less than hydrogen (0.09 kg/m^3)
		float DensityKGPerCubicUU = 1.0f;
		if (UPhysicalMaterial* PhysMat = OwningBodyInstance->GetSimplePhysicalMaterial())
		{
			DensityKGPerCubicUU = FMath::Max(KgPerM3ToKgPerCm3(0.09f), gPerCm3ToKgPerCm3(PhysMat->Density));
		}
		return DensityKGPerCubicUU;
	}

	Chaos::FMassProperties ApplyMassPropertiesModifiers(const FBodyInstance* OwningBodyInstance, Chaos::FMassProperties MassProps, const FTransform& MassModifierTransform, const bool bInertaScaleIncludeMass)
	{
		float OldMass = MassProps.Mass;
		float NewMass = 0.f;

		if (OwningBodyInstance->bOverrideMass == false)
		{
			// The mass was calculated assuming uniform density. RaiseMassToPower for values of less than 1.0
			// is used to correct this for objects where the density is higher closer to the surface.
			float RaiseMassToPower = 0.75f;
			if (UPhysicalMaterial* PhysMat = OwningBodyInstance->GetSimplePhysicalMaterial())
			{
				RaiseMassToPower = PhysMat->RaiseMassToPower;
			}

			float UsePow = FMath::Clamp<float>(RaiseMassToPower, UE_KINDA_SMALL_NUMBER, 1.f);
			NewMass = FMath::Pow(OldMass, UsePow);

			// Apply user-defined mass scaling.
			NewMass = FMath::Max(OwningBodyInstance->MassScale * NewMass, 0.001f);	//min weight of 1g
		}
		else
		{
			NewMass = FMath::Max(OwningBodyInstance->GetMassOverride(), 0.001f);	//min weight of 1g
		}

		float MassRatio = NewMass / OldMass;
		MassProps.Mass *= MassRatio;
		MassProps.InertiaTensor *= MassRatio;
		//COM Nudge is done in actor space. 
		Chaos::FVec3 COMNudgeInActorSpace = MassModifierTransform.TransformVector(OwningBodyInstance->COMNudge);
		MassProps.CenterOfMass += COMNudgeInActorSpace;

		// Scale the inertia tensor by the owning body instance's InertiaTensorScale
		// NOTE: PhysX scales the inertia by the mass increase we would get from the scale change, even though we 
		// don't actually scale the mass at all based on InertiaScale. This is non-intuituve. E.g., you may expect 
		// that if InertiaScale = (S,S,S) and the mass is fixed (we already accounted for the effect of mass change on
		// ont the inertia just above), then the inertia components would roughly multiply by S^2, but actually
		// they end up multiplied by S^5. 
		// The option we choose is controlled by bInertaScaleIncludeMass.
		//		bInertaScaleIncludeMass = true: original behaviour as in PhysX
		//		bInertaScaleIncludeMass = false: more sensible behaviour given that InertiaScale does not affect mass
		if (!(OwningBodyInstance->InertiaTensorScale - FVector::OneVector).IsNearlyZero(1e-3f))
		{
			MassProps.InertiaTensor = Chaos::Utilities::ComputeWorldSpaceInertia(MassProps.RotationOfMass, MassProps.InertiaTensor);
			MassProps.RotationOfMass = Chaos::FRotation3::FromIdentity();
			MassProps.InertiaTensor = Chaos::Utilities::ScaleInertia(MassProps.InertiaTensor, OwningBodyInstance->InertiaTensorScale, bInertaScaleIncludeMass);
			TransformToLocalSpace(MassProps);
		}

		// If we move the center of mass, we need to update the inertia using parallel-axis theorem. If we don't do this
		// and the center of mass is moved significantly it can cause jitter (inertia too small for the contact positions)
		// NOTE: This must come after ScaleInertia because ScaleInertia effectively calculates the "equivalent box" dimensions
		// which is not always possible, e.g., if we move the CoM outside of a box you will get negative elements in the scaled inertia!
		if (CVars::bPhysicsComNudgeAdjustInertia)
		{
			MassProps.InertiaTensor = Chaos::Utilities::ComputeWorldSpaceInertia(MassProps.RotationOfMass, MassProps.InertiaTensor);
			MassProps.RotationOfMass = Chaos::FRotation3::FromIdentity();

			MassProps.InertiaTensor.M[0][0] += MassProps.Mass * COMNudgeInActorSpace.X * COMNudgeInActorSpace.X;
			MassProps.InertiaTensor.M[1][1] += MassProps.Mass * COMNudgeInActorSpace.Y * COMNudgeInActorSpace.Y;
			MassProps.InertiaTensor.M[2][2] += MassProps.Mass * COMNudgeInActorSpace.Z * COMNudgeInActorSpace.Z;

			TransformToLocalSpace(MassProps);
		}

		return MassProps;
	}

	Chaos::FMassProperties ComputeMassProperties(const FBodyInstance* OwningBodyInstance, const TArray<FPhysicsShapeHandle>& Shapes, const FTransform& MassModifierTransform, const bool bInertaScaleIncludeMass)
	{
		// Calculate the mass properties based on the shapes assuming uniform density
		Chaos::FMassProperties MassProps;
		ChaosInterface::CalculateMassPropertiesFromShapeCollection(MassProps, Shapes, GetBodyInstanceDensity(OwningBodyInstance));

		// Diagonalize the inertia
		Chaos::TransformToLocalSpace(MassProps);

		// Apply the BodyInstance's mass and inertia modifiers
		return ApplyMassPropertiesModifiers(OwningBodyInstance, MassProps, MassModifierTransform, bInertaScaleIncludeMass);
	}

	Chaos::FMassProperties ComputeMassProperties(const FBodyInstance* OwningBodyInstance, const Chaos::FShapesArray& Shapes, const TArray<bool>& bContributesToMass, const FTransform& MassModifierTransform, const bool bInertaScaleIncludeMass)
	{
		// Calculate the mass properties based on the shapes assuming uniform density
		Chaos::FMassProperties MassProps;
		ChaosInterface::CalculateMassPropertiesFromShapeCollection(MassProps, Shapes, bContributesToMass, GetBodyInstanceDensity(OwningBodyInstance));

		// Diagonalize the inertia
		Chaos::TransformToLocalSpace(MassProps);

		// Apply the BodyInstance's mass and inertia modifiers
		return ApplyMassPropertiesModifiers(OwningBodyInstance, MassProps, MassModifierTransform, bInertaScaleIncludeMass);
	}
}
