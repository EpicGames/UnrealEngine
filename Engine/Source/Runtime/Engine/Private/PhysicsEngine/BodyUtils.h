// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Chaos/ShapeInstanceFwd.h"
#include "Containers/Array.h"
#include "Containers/ContainerAllocationPolicies.h"
#include "PhysicsInterfaceDeclaresCore.h"

struct FBodyInstance; 

namespace Chaos
{
	struct FMassProperties;
}

/**
 * Utility methods for use by BodyInstance and ImmediatePhysics
 */
namespace BodyUtils
{
	/** 
	 * Computes and adds the mass properties (inertia, com, etc...) based on the mass settings of the body instance. 
	 * The inertia returned will be diagonal, and there may be a non-identity rotation of mass.
	 * Note: this includes a call to ModifyMassProperties, so the BodyInstance modifiers will be included in the calculation.
	 */
	Chaos::FMassProperties ComputeMassProperties(const FBodyInstance* OwningBodyInstance, const TArray<FPhysicsShapeHandle>& Shapes, const FTransform& MassModifierTransform, const bool bInertaScaleIncludeMass = false);
	Chaos::FMassProperties ComputeMassProperties(const FBodyInstance* OwningBodyInstance, const Chaos::FShapesArray& Shapes, const TArray<bool>& bContributesToMass, const FTransform& MassModifierTransform, const bool bInertaScaleIncludeMass = false);
}