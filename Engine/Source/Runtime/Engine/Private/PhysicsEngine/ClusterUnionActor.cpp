// Copyright Epic Games, Inc. All Rights Reserved.
#include "PhysicsEngine/ClusterUnionActor.h"

#include "PhysicsEngine/ClusterUnionComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ClusterUnionActor)

AClusterUnionActor::AClusterUnionActor(const FObjectInitializer& ObjectInitializer)
	: AActor(ObjectInitializer)
{
	ClusterUnion = CreateDefaultSubobject<UClusterUnionComponent>(TEXT("ClusterUnion"));
	SetRootComponent(ClusterUnion);

	SetRemoteRoleForBackwardsCompat(ROLE_SimulatedProxy);
	bReplicates = true;
	SetReplicatingMovement(true);
}