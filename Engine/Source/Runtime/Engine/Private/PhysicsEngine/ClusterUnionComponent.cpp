// Copyright Epic Games, Inc. All Rights Reserved.
#include "PhysicsEngine/ClusterUnionComponent.h"

#include "Chaos/ImplicitObject.h"
#include "Chaos/Serializable.h"
#include "Chaos/ShapeInstance.h"
#include "Engine/OverlapResult.h"
#include "Engine/World.h"
#include "Net/Core/PushModel/PushModel.h"
#include "Net/UnrealNetwork.h"
#include "Physics/Experimental/PhysScene_Chaos.h"
#include "Physics/GenericPhysicsInterface.h"
#include "Physics/PhysicsReplicationQuantization.h"
#include "PhysicsEngine/ClusterUnionReplicatedProxyComponent.h"
#include "PhysicsEngine/ExternalSpatialAccelerationPayload.h"
#include "PhysicsEngine/PhysicsObjectExternalInterface.h"
#include "PhysicsProxy/ClusterUnionPhysicsProxy.h"
#include "TimerManager.h"
#include "PhysicsEngine/PhysicsSettings.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ClusterUnionComponent)

DEFINE_LOG_CATEGORY(LogClusterUnion);

namespace
{
	bool bUseClusterUnionAccelerationStructure = true;
	FAutoConsoleVariableRef CVarUseClusterUnionAccelerationStructure(TEXT("ClusterUnion.UseAccelerationStructure"), bUseClusterUnionAccelerationStructure, TEXT("Whether component level sweeps and overlaps against cluster unions should use an acceleration structure instead."));

	bool bUseLocalRoleForAuthorityCheck = true;
	FAutoConsoleVariableRef CVarUseLocalRoleForAuthorityCheck(TEXT("ClusterUnion.UseLocalRoleForAuthorityCheck"), bUseLocalRoleForAuthorityCheck, TEXT("If true, we will only check this component's owner local role to determine authority"));

	bool bPreAllocateLocalBoneDataMap = true;
	FAutoConsoleVariableRef CVarPreAllocateLocalBoneDataMap(TEXT("ClusterUnion.PreAllocateLocalBoneDataMap"), bPreAllocateLocalBoneDataMap, TEXT("If true, it will reserve an expected size for the local map used to cache updated bone data"));

	float LocalBoneDataMapGrowFactor = 1.2f;
	FAutoConsoleVariableRef CVarLocalBoneDataMapGrowFactor(TEXT("ClusterUnion.LocalBoneDataMapGrowFactor"), LocalBoneDataMapGrowFactor, TEXT("Grow factor to apply to the size of bone data array of pre-existing component when preallocating the local bones data map"));

	bool bApplyReplicatedRigidStateOnCreatePhysicsState = true;
	FAutoConsoleVariableRef CVarApplyReplicatedRigidStateOnCreatePhysicsState(TEXT("ClusterUnion.ApplyReplicatedRigidStateOnCreatePhysicsState"), bApplyReplicatedRigidStateOnCreatePhysicsState, TEXT("When physics state is created, apply replicated rigid state. Useful because sometimes the initial OnRep will have been called before a proxy exists, so initial properties will be unset"));

	bool bDirtyRigidStateOnlyIfChanged = false;
	FAutoConsoleVariableRef CVarDirtyRigidStateOnlyIfChanged(TEXT("ClusterUnion.DirtyRigidStateOnlyIfChanged"), bDirtyRigidStateOnlyIfChanged, TEXT("Add a check for changed rigid state before marking it dirty and updating the replicated data. No need to flush an update if there was no change."));

	bool bFlushNetDormancyOnSyncProxy = true;
	FAutoConsoleVariableRef CVarFlushNetDormancyOnSyncProxy(TEXT("ClusterUnion.FlushNetDormancyOnSyncProxy"), bFlushNetDormancyOnSyncProxy, TEXT("When there is a new rigid state on the authority, flush net dormancy so that even if this object is net dorman the rigid state will come through to the client."));

	bool GSkipZeroStateInOnRep = true;
	FAutoConsoleVariableRef CVarSkipZeroState(TEXT("ClusterUnion.SkipZeroStateInOnRep"), GSkipZeroStateInOnRep, TEXT("Whether we skip 0 (uninitialized) states when running the onrep for the replicated rigid state of the cluster union"));

	template<typename PayloadType>
	struct TClusterUnionAABBTreeStorageTraits
	{
		constexpr static uint32 HashTableBucketsSize = 256;

		using PayloadToInfoType = Chaos::TSQMap<PayloadType, Chaos::FAABBTreePayloadInfo>;

		static void InitPayloadToInfo(PayloadToInfoType& PayloadToInfo)
		{
			ensureMsgf(PayloadToInfo.Num() == 0, TEXT("Expected an empty map in InitPayloadToInfo, this will incur a rehash of %d elems"), PayloadToInfo.Num());
			PayloadToInfo.ResizeHashBuckets(HashTableBucketsSize);
		}
	};

	// TODO: Should this be exposed in Chaos instead?
	using FAccelerationStructure = Chaos::TAABBTree<
		FExternalSpatialAccelerationPayload, 
		Chaos::TAABBTreeLeafArray<FExternalSpatialAccelerationPayload>,
		true,
		Chaos::FReal,
		TClusterUnionAABBTreeStorageTraits<FExternalSpatialAccelerationPayload>>;

	TUniquePtr<Chaos::ISpatialAcceleration<FExternalSpatialAccelerationPayload, Chaos::FReal, 3>> CreateEmptyAccelerationStructure()
	{
		FAccelerationStructure* Structure = new FAccelerationStructure{
			FAccelerationStructure::EmptyInit{},
			FAccelerationStructure::DefaultMaxChildrenInLeaf,
			FAccelerationStructure::DefaultMaxTreeDepth,
			FAccelerationStructure::DefaultMaxPayloadBounds,
			FAccelerationStructure::DefaultMaxNumToProcess,
			true,
			false
		};

		check(Structure != nullptr);
		return TUniquePtr<Chaos::ISpatialAcceleration<FExternalSpatialAccelerationPayload, Chaos::FReal, 3>>(Structure);
	}

	FBox GetWorldBoundsForParticle(const Chaos::FPBDRigidParticle* Particle)
	{
		if (!Particle)
		{
			return FBox(ForceInit);
		}

		if (const Chaos::FImplicitObjectRef Geometry = Particle->GetGeometry(); Geometry && Geometry->HasBoundingBox())
		{
			const Chaos::FAABB3 WorldBox = Geometry->CalculateTransformedBounds(Chaos::TRigidTransform<Chaos::FReal, 3>(Particle->X(), Particle->R()));
			return FBox{ WorldBox.Min(), WorldBox.Max() };
		}

		return FBox(ForceInit);
	}
}

UClusterUnionComponent::UClusterUnionComponent(const FObjectInitializer& ObjectInitializer)
	: UPrimitiveComponent(ObjectInitializer)
{
	PhysicsProxy = nullptr;
	SetIsReplicatedByDefault(true);
	bComputeBoundsOnceForGame = false;
	bHasCachedLocalBounds = false;
#if WITH_EDITORONLY_DATA
	bVisualizeComponent = true;
#endif
	GravityGroupIndexOverride = INDEX_NONE;
	bEnableDamageFromCollision = true;
}

UClusterUnionComponent::~UClusterUnionComponent()
{
	ensureMsgf(
		!PhysicsProxy,
		TEXT("ClusterUnionComponent %s destroyed while still attached to physics proxy. "
			"BeginDestroy() might be overridden without a base class call."),
			*GetName());
}

FPhysScene_Chaos* UClusterUnionComponent::GetChaosScene() const
{
	if (AActor* Owner = GetOwner(); Owner && Owner->GetWorld())
	{
		return Owner->GetWorld()->GetPhysicsScene();
	}

	check(GWorld);
	return GWorld->GetPhysicsScene();
}

DECLARE_CYCLE_STAT(TEXT("UClusterUnionComponent::AddComponentToCluster"), STAT_ClusterUnionComponent_AddComponentToCluster, STATGROUP_Chaos);
void UClusterUnionComponent::AddComponentToCluster(UPrimitiveComponent* InComponent, const TArray<int32>& BoneIds, bool bRebuildGeometry)
{
	SCOPE_CYCLE_COUNTER(STAT_ClusterUnionComponent_AddComponentToCluster);

	if (!InComponent || !PhysicsProxy || !InComponent->GetWorld())
	{
		return;
	}

	TObjectKey<UPrimitiveComponent> ComponentKey(InComponent);

	FClusterUnionPendingAddData PendingData;

	if (!InComponent->HasValidPhysicsState())
	{
		if (!PendingComponentsToAdd.Contains(ComponentKey))
		{
			Algo::Transform(BoneIds, PendingData.BonesData, [](int32 BoneID){ return FClusterUnionBoneData(BoneID); });

			// Early out - defer adding the component to the cluster until the component has a valid physics state.
			PendingComponentsToAdd.Add(ComponentKey, PendingData);
			InComponent->OnComponentPhysicsStateChanged.AddDynamic(this, &UClusterUnionComponent::HandleComponentPhysicsStateChange);
		}
		return;
	}

	PendingComponentsToAdd.Remove(ComponentKey);

	TArray<Chaos::FPhysicsObjectHandle> AllObjects = InComponent->GetAllPhysicsObjects();
	FLockedWritePhysicsObjectExternalInterface Interface = FPhysicsObjectExternalInterface::LockWrite(AllObjects);

	TArray<Chaos::FPhysicsObjectHandle> Objects;
	if (BoneIds.IsEmpty())
	{
		Objects = AllObjects;
	}
	else
	{
		Objects.Reserve(BoneIds.Num());
		for (int32 Id : BoneIds)
		{
			Objects.Add(InComponent->GetPhysicsObjectById(Id));
		}
	}

	if (BoneIds.IsEmpty())
	{
		Objects = Objects.FilterByPredicate(
			[&Interface](Chaos::FPhysicsObjectHandle Handle)
			{
				return !Interface->AreAllDisabled({ &Handle, 1 });
			}
		);
	}

	if (Objects.IsEmpty())
	{
		UE_LOGF(LogClusterUnion, Warning, "Trying to add a component [%p] with no physics objects to a cluster union...ignoring", InComponent)
		return;
	}

	PendingData.BonesData.Empty(Objects.Num());

	// We want to fix the ChildToParent relationship between the particles we're adding and the cluster union
	// based on the current game-thread state, not the future physics-thread state as the two may diverge
	// which could cause the alignment between the particle and the cluster union to be different from what
	// would be expected.
	TArray<FTransform> ChildToParents;
	ChildToParents.Reserve(Objects.Num());

	const FTransform CurrentComponentTransform = GetComponentTransform();

	const bool bHasKinematicObject = !Interface->AreAllDynamicOrSleeping(Objects);
	for (Chaos::FPhysicsObjectHandle Object : Objects)
	{
		if (Chaos::FPBDRigidParticle* Particle = Interface->GetRigidParticle(Object))
		{
			const int32 BoneId = Chaos::FPhysicsObjectInterface::GetId(Object);
			if (BoneId == INDEX_NONE)
			{
				continue;
			}
			FClusterUnionBoneData BoneData;
			BoneData.ID = BoneId;
			BoneData.ParticleID = Particle ? Particle->UniqueIdx() : Chaos::FUniqueIdx();
			
			PendingData.BonesData.Add(BoneData);

			if (AccelerationStructure)
			{
				const FBox HandleBounds = GetWorldBoundsForParticle(Particle);
				FExternalSpatialAccelerationPayload Handle;
				Handle.Initialize(ComponentKey, BoneData.ID, BoneData.ParticleID);
				if (ensure(Handle.IsValid()))
				{
					AccelerationStructure->UpdateElement(Handle, Chaos::TAABB<Chaos::FReal, 3>{HandleBounds.Min, HandleBounds.Max}, HandleBounds.IsValid != 0);
				}
			}

			const FTransform CurrentParticleTransform = Interface->GetTransform(Object);
			ChildToParents.Add(CurrentParticleTransform.GetRelativeTransform(CurrentComponentTransform));
		}
	}

	if (!ChildToParents.IsEmpty())
	{
		// The first time we add a component, we need to manage the GT-side object state and set it to kinematic immediately.
		if (bHasKinematicObject && PendingComponentSync.IsEmpty() && PerComponentData.IsEmpty())
		{
			Chaos::FPhysicsObjectHandle SelfHandle = PhysicsProxy->GetPhysicsObjectHandle();
			Interface->ForceKinematic( { &SelfHandle, 1 });
		}

		// Need to listen to changes in the component's physics state. If it gets destroyed it should be removed from the cluster union as well.
		// This technically is giving us a false sense of security because when we get this notification, the primitive component will already have no physics proxy on it so we can't actually grab physics objects/particles from it.
		InComponent->OnComponentPhysicsStateChanged.AddUniqueDynamic(this, &UClusterUnionComponent::HandleComponentPhysicsStateChangePostAddIntoClusterUnion);

		if (bRebuildGeometry)
		{
			AddGTParticleGeometry(Objects);
		}

		PendingComponentSync.Add(ComponentKey, PendingData);

		PhysicsProxy->AddPhysicsObjects_External(Objects);

		TArray<int32> BoneIDs;
		GetBoneIDsFromComponentData(BoneIDs, PendingData);
		ForceSetChildToParent(InComponent, BoneIDs, ChildToParents);
	}
}

DECLARE_CYCLE_STAT(TEXT("UClusterUnionComponent::RemoveComponentFromCluster"), STAT_ClusterUnionComponent_RemoveComponentFromCluster, STATGROUP_Chaos);
void UClusterUnionComponent::RemoveComponentFromCluster(UPrimitiveComponent* InComponent)
{
	SCOPE_CYCLE_COUNTER(STAT_ClusterUnionComponent_RemoveComponentFromCluster);

	if (!InComponent || !PhysicsProxy)
	{
		return;
	}

	const TObjectKey<UPrimitiveComponent> RemovedComponentKey(InComponent);
	const int32 NumRemoved = PendingComponentsToAdd.Remove(RemovedComponentKey);
	if (NumRemoved > 0)
	{
		// We haven't actually added yet so we can early out.
		return;
	}

	// If we're still waiting on the physics sync for the added component, we still need to
	// remove its acceleration handles from the acceleration structure since the payloads have
	// already been added and will not yet be stored in PerComponentData.
	if (FClusterUnionPendingAddData* PendingData = PendingComponentSync.Find(InComponent))
	{
		if (AccelerationStructure)
		{
			for (const FClusterUnionBoneData BoneData : PendingData->BonesData)
			{
				FExternalSpatialAccelerationPayload Payload;
				Payload.Initialize(RemovedComponentKey, BoneData.ID, BoneData.ParticleID);
				if (ensure(Payload.IsValid()))
				{
					AccelerationStructure->RemoveElement(Payload);
				}
			}
		}

		PendingComponentSync.Remove(InComponent);
	}

	TSet<Chaos::FPhysicsObjectHandle> PhysicsObjectsToRemove;

	FLockedWritePhysicsObjectExternalInterface Interface = FPhysicsObjectExternalInterface::LockWrite(GetWorld()->GetPhysicsScene());

	if (FClusteredComponentData* ComponentData = PerComponentData.Find(InComponent))
	{
		// We need to mark the replicated proxy as pending deletion.
		// This way anyone who tries to use the replicated proxy component knows that it
		// doesn't actually denote a meaningful cluster union relationship.
		ComponentData->bPendingDeletion = true;

		if (InComponent->HasValidPhysicsState())
		{
			TArray<int32> BoneIDs;
			GetBoneIDsFromComponentData(BoneIDs, *ComponentData);
			PhysicsObjectsToRemove = TSet<Chaos::FPhysicsObjectHandle>{ GetAllPhysicsObjectsById(InComponent, BoneIDs) };
		}

		if (IsAuthority())
		{
			if (UClusterUnionReplicatedProxyComponent* Component = ComponentData->ReplicatedProxyComponent.Get())
			{
				Component->MarkPendingDeletion();
			}
		}

		if (AccelerationStructure)
		{
			for (const FClusterUnionBoneData& BoneData : ComponentData->BonesData)
			{
				FExternalSpatialAccelerationPayload Payload;
				Payload.Initialize(RemovedComponentKey, BoneData.ID, BoneData.ParticleID);
				if (ensure(Payload.IsValid()))
				{
					AccelerationStructure->RemoveElement(Payload);
				}
			}
		}

		// If PhysicsObjectsToRemove is empty, it either means that BoneIds is empty OR the component's physics state is already destroyed.
		// In the case of the latter, we rely on the physics thread to cleanup the cluster union manager properly and to sync back.
		if (!PhysicsObjectsToRemove.IsEmpty())
		{
			PhysicsProxy->RemovePhysicsObjects_External(PhysicsObjectsToRemove);
			RemoveGTParticleGeometry(PhysicsObjectsToRemove);
		}
	}
}

// TODO: Can this merge with RemoveComponentFromCluster?
void UClusterUnionComponent::RemoveComponentBonesFromCluster(UPrimitiveComponent* InComponent, const TArray<int32>& BoneIds)
{
	if (!InComponent || !PhysicsProxy)
	{
		return;
	}

	TObjectKey<UPrimitiveComponent> ComponentKey(InComponent);

	// If we're removing bones from a component and it hasn't been added into the component yet, then we want to make sure 
	// the set we actually want to add doesn't have the remove bones in it.
	if (FClusterUnionPendingAddData* PendingAdd = PendingComponentsToAdd.Find(ComponentKey))
	{
		for (int32 BoneId : BoneIds)
		{
			PendingAdd->BonesData.Remove(FClusterUnionBoneData(BoneId));
		}

		if (PendingAdd->BonesData.IsEmpty())
		{
			PendingComponentsToAdd.Remove(ComponentKey);
		}

		// Component is pending add so we can be confident we don't need to do any more work.
		return;
	}

	// If we're still waiting on the physics sync for the added component, we still need to
	// remove its acceleration handles from the acceleration structure since the payloads have
	// already been added and will not yet be stored in PerComponentData.
	if (FClusterUnionPendingAddData* PendingData = PendingComponentSync.Find(ComponentKey))
	{
		if (AccelerationStructure)
		{
			for (int32 BoneId : BoneIds)
			{
				FExternalSpatialAccelerationPayload Payload;
				Payload.Initialize(ComponentKey, BoneId);
				if (ensure(Payload.IsValid()))
				{
					AccelerationStructure->RemoveElement(Payload);
				}
			}
		}

		for (int32 BoneId : BoneIds)
		{
			PendingData->BonesData.Remove(FClusterUnionBoneData(BoneId));
		}

		if (PendingData->BonesData.IsEmpty())
		{
			PendingComponentSync.Remove(ComponentKey);
		}
	}

	TSet<Chaos::FPhysicsObjectHandle> PhysicsObjectsToRemove;
	if (FClusteredComponentData* ComponentData = PerComponentData.Find(ComponentKey))
	{
		for (TArray<FClusterUnionBoneData>::TIterator RemoveBoneDataIterator = ComponentData->BonesData.CreateIterator(); RemoveBoneDataIterator; ++RemoveBoneDataIterator)
		{
			if (AccelerationStructure)
			{
				FExternalSpatialAccelerationPayload Payload;
				Payload.Initialize(InComponent, RemoveBoneDataIterator->ID, RemoveBoneDataIterator->ParticleID);
				if (ensure(Payload.IsValid()))
				{
					AccelerationStructure->RemoveElement(Payload);
				}
			}

			RemoveBoneDataIterator.RemoveCurrent();
		}

		if (InComponent->HasValidPhysicsState())
		{
			PhysicsObjectsToRemove = TSet<Chaos::FPhysicsObjectHandle>{ GetAllPhysicsObjectsById(InComponent, BoneIds) };
		}

		// If PhysicsObjectsToRemove is empty, it either means that BoneIds is empty OR the component's physics state is already destroyed.
		// In the case of the latter, we rely on the physics thread to cleanup the cluster union manager properly and to sync back.
		if (!PhysicsObjectsToRemove.IsEmpty())
		{
			PhysicsProxy->RemovePhysicsObjects_External(PhysicsObjectsToRemove);
			RemoveGTParticleGeometry(PhysicsObjectsToRemove);
		}

		if (ComponentData->BonesData.IsEmpty())
		{
			// We need to mark the replicated proxy as pending deletion.
			// This way anyone who tries to use the replicated proxy component knows that it
			// doesn't actually denote a meaningful cluster union relationship.
			ComponentData->bPendingDeletion = true;

			if (IsAuthority())
			{
				if (UClusterUnionReplicatedProxyComponent* Component = ComponentData->ReplicatedProxyComponent.Get())
				{
					Component->MarkPendingDeletion();
				}
			}

			PerComponentData.Remove(ComponentKey);
		}

	}
}

void UClusterUnionComponent::AddGTParticleGeometry(const TArray<Chaos::FPhysicsObjectHandle>& PhysicsObjects)
{
	if (!PhysicsProxy)
	{
		return;
	}

	// This is an assumption that all the physics objects are part of this scene and thus this is the right thing to lock.
	FLockedWritePhysicsObjectExternalInterface Interface = FPhysicsObjectExternalInterface::LockWrite(GetWorld()->GetPhysicsScene());

	TArray<Chaos::FImplicitObjectPtr> ImplicitObjects;
	TArray<Chaos::FPBDRigidParticle*> ShapeParticles;
	
	const FTransform ClusterWorldTM = GetComponentTransform();
	for (Chaos::FPhysicsObjectHandle PhysicsObject : PhysicsObjects)
	{
		const int32 BoneId = Chaos::FPhysicsObjectInterface::GetId(PhysicsObject);
		if (BoneId == INDEX_NONE)
		{
			continue;
		}

		Chaos::FPBDRigidParticle* RigidParticle = Interface->GetRigidParticle(PhysicsObject);
		if (RigidParticle && RigidParticle->GetGeometry())
		{
			const FTransform ChildWorldTM{ RigidParticle->R(), RigidParticle->X() };
			const FTransform Frame = ChildWorldTM.GetRelativeTransform(ClusterWorldTM);
			ImplicitObjects.Add(Chaos::FImplicitObjectPtr(Chaos::FClusterUnionManager::CreateTransformGeometryForClusterUnion<Chaos::EThreadContext::External>(RigidParticle, Frame)));
			ShapeParticles.Add(RigidParticle);
		}
	}
	if(!ImplicitObjects.IsEmpty() && PhysicsProxy->GetParticle_External())
	{
		const Chaos::FImplicitObjectRef ExistingGeometry = PhysicsProxy->GetParticle_External()->GetGeometry();
		if(ExistingGeometry == nullptr || ExistingGeometry->GetType() != Chaos::ImplicitObjectType::Union)
		{
			Chaos::FImplicitObjectUnion* NewGeometry = ImplicitObjects.IsEmpty() ? new Chaos::FImplicitObjectUnionClustered() : new Chaos::FImplicitObjectUnion(MoveTemp(ImplicitObjects));
			NewGeometry->SetAllowBVH(true);
    
			PhysicsProxy->SetGeometry_External(Chaos::FImplicitObjectPtr(NewGeometry), ShapeParticles);
		}
		else
		{
			PhysicsProxy->MergeGeometry_External(MoveTemp(ImplicitObjects), ShapeParticles);
		}
	}
}

void UClusterUnionComponent::RemoveGTParticleGeometry(const TSet<Chaos::FPhysicsObjectHandle>& PhysicsObjects)
{
	if (!PhysicsProxy)
	{
		return;
	}
	TArray<Chaos::FPhysicsObjectHandle> ArrayObjects;
	ArrayObjects.SetNum(PhysicsObjects.Num());

	for(Chaos::FPhysicsObjectHandle PhysicsObject : PhysicsObjects)
	{
		ArrayObjects.Add(PhysicsObject);
	}

	// This is an assumption that all the physics objects are part of this scene and thus this is the right thing to lock.
	FLockedWritePhysicsObjectExternalInterface Interface = FPhysicsObjectExternalInterface::LockWrite(GetWorld()->GetPhysicsScene());

	TArray<Chaos::FPBDRigidParticle*> ShapeParticles;
	for (Chaos::FPhysicsObjectHandle PhysicsObject : PhysicsObjects)
	{
		const int32 BoneId = Chaos::FPhysicsObjectInterface::GetId(PhysicsObject);
		if (BoneId == INDEX_NONE)
		{
			continue;
		}

		Chaos::FPBDRigidParticle* RigidParticle = Interface->GetRigidParticle(PhysicsObject);
		if (RigidParticle && RigidParticle->GetGeometry())
		{
			ShapeParticles.Add(RigidParticle);

		}
	}
	if(!ShapeParticles.IsEmpty() && PhysicsProxy->GetParticle_External())
	{
		PhysicsProxy->RemoveShapes_External(ShapeParticles);
	}
}

void UClusterUnionComponent::ForceRebuildGTParticleGeometry()
{
	if (!PhysicsProxy)
	{
		return;
	}

	// This is an assumption that all the physics objects are part of this scene and thus this is the right thing to lock.
	FLockedReadPhysicsObjectExternalInterface Interface = FPhysicsObjectExternalInterface::LockRead(GetWorld()->GetPhysicsScene());

	const FTransform ClusterWorldTM = GetComponentTransform();
	TArray<UPrimitiveComponent*> Components = GetAllCurrentChildComponents();
	// This code is similar-ish to FClusterUnionManager::ForceRecreateClusterUnionGeometry but not enough to make it necessary
	// to reshare exactly the same code.
	TArray<Chaos::FImplicitObjectPtr> Objects;
	TArray<Chaos::FPBDRigidParticle*> Particles;

	// Should be a good number to reserve for now since it's a generally safe assumption we'll be working with 1 particle per component added.
	Objects.Reserve(Components.Num());
	Particles.Reserve(Components.Num());

	for (UPrimitiveComponent* Child : Components)
	{
		check(Child != nullptr);
		TArray<int32> BoneIds = GetAddedBoneIdsForComponent(Child);
		TArray<Chaos::FPhysicsObjectHandle> PhysicsObjects;
		PhysicsObjects.Reserve(BoneIds.Num());
		for (int32 Id : BoneIds)
		{
			PhysicsObjects.Add(Child->GetPhysicsObjectById(Id));
		}

		for (Chaos::FPBDRigidParticle* Particle : Interface->GetAllRigidParticles(PhysicsObjects))
		{
			if (Particle && Particle->GetGeometry())
			{
				const FTransform ChildWorldTM{ Particle->R(), Particle->X() };
				const FTransform Frame = ChildWorldTM.GetRelativeTransform(ClusterWorldTM);
				Objects.Add(Chaos::FImplicitObjectPtr(Chaos::FClusterUnionManager::CreateTransformGeometryForClusterUnion<Chaos::EThreadContext::External>(Particle, Frame)));
				Particles.Add(Particle);
			}
		}
	}

	// TODO: Make sure that the empty cluster union does not get added to the acceleration structure
	if (Particles.Num() == 0)
	{
		return;
	}	

	Chaos::FImplicitObjectUnion* NewGeometry = Objects.IsEmpty() ? new Chaos::FImplicitObjectUnionClustered() : new Chaos::FImplicitObjectUnion(MoveTemp(Objects));
	NewGeometry->SetAllowBVH(true);

	PhysicsProxy->SetGeometry_External(Chaos::FImplicitObjectPtr(NewGeometry), Particles); 
}

const UClusterUnionComponent::FSpatialAcceleration* UClusterUnionComponent::GetSpatialAcceleration() const
{
	return AccelerationStructure.Get();
}

TArray<UPrimitiveComponent*> UClusterUnionComponent::GetPrimitiveComponents()
{
	TArray<UPrimitiveComponent*> PrimitiveComponents;
	PrimitiveComponents.Reserve(PerComponentData.Num());

	for (auto Iter = PerComponentData.CreateIterator(); Iter; ++Iter)
	{
		PrimitiveComponents.Add(Iter.Key().ResolveObjectPtr());
	}

	return PrimitiveComponents;
}

TArray<AActor*> UClusterUnionComponent::GetActors()
{
	TArray<AActor*> Actors;
	Actors.Reserve(ActorToComponents.Num());

	for (auto Iter = ActorToComponents.CreateIterator(); Iter; ++Iter)
	{
		Actors.Add(Iter.Key().ResolveObjectPtr());
	}

	return Actors;
}

void UClusterUnionComponent::SetIsAnchored(bool bIsAnchored)
{
	// deprecated - should not be called anymore
	ensure(false);
	if (!PhysicsProxy)
	{
		return;
	}

	PhysicsProxy->SetIsAnchored_External(bIsAnchored);
}

ENGINE_API bool UClusterUnionComponent::IsAnchored() const
{
	if (!PhysicsProxy)
	{
		return false;
	}

	return PhysicsProxy->IsAnchored_External();
}

void UClusterUnionComponent::SetEnableDamageFromCollision(bool bValue)
{
	bEnableDamageFromCollision = bValue;

	if (PhysicsProxy)
	{
		PhysicsProxy->SetEnableStrainOnCollision_External(bValue);
	}
}

bool UClusterUnionComponent::IsAuthority() const
{
	if (bUseLocalRoleForAuthorityCheck)
	{
		if (AActor* Owner = GetOwner())
        {
        	return Owner->GetLocalRole() == ROLE_Authority;
        }

		return false;
	}

	ENetMode Mode = GetNetMode();
	if (Mode == ENetMode::NM_Standalone)
	{
		return true;
	}

	if (AActor* Owner = GetOwner())
	{
		return Owner->GetLocalRole() == ROLE_Authority && Mode != NM_Client;
	}

	return false;
}

void UClusterUnionComponent::OnCreatePhysicsState()
{
	USceneComponent::OnCreatePhysicsState();

	// If we've already created the physics proxy we shouldn't do this again.
	if (PhysicsProxy)
	{
		return;
	}

	// We need to set some properties on the BodyInstance to get things to play nice.
	BodyInstance.OwnerComponent = this;

	// If we're not actually playing/needing this to simulate (e.g. in the editor) there should be no reason to create this proxy.
	const bool bValidWorld = GetWorld() && (GetWorld()->IsGameWorld() || GetWorld()->IsPreviewWorld());
	if (!bValidWorld)
	{
		return;
	}

	const FPhysicsPredictionSettings& PhysicsPredictionSettings = UPhysicsSettings::Get()->PhysicsPrediction;
	bPhysicsPredictionEnabled = PhysicsPredictionSettings.bEnablePhysicsPrediction;

	// TODO: Expose these parameters via the component.
	Chaos::FClusterCreationParameters Parameters{ 0.3f, 100, false, false };
	Parameters.ConnectionMethod = Chaos::FClusterCreationParameters::EConnectionMethod::PointImplicit;
	Parameters.bEnableStrainOnCollision = bEnableDamageFromCollision;

	FChaosUserData::Set<UPrimitiveComponent>(&PhysicsUserData, this);

	const bool bHasAuthority = GetOwner()->HasAuthority();

	Chaos::FClusterUnionInitData InitData;
	InitData.UserData = static_cast<void*>(&PhysicsUserData);
	InitData.ActorId = GetOwner()->GetUniqueID();
	InitData.ComponentId = GetUniqueID();
	InitData.InitialTransform = GetComponentTransform();
	InitData.GravityGroupOverride = GravityGroupIndexOverride;
	InitData.bCCDEnabled = BodyInstance.bUseCCD;
	InitData.bMACDEnabled = BodyInstance.GetUseMACD();

	// Client needs to be set to unbreakable so the server is authoritative.
	InitData.bUnbreakable = !bHasAuthority;

	// Only need to check connectivity on the server and have the client rely on replication to get the memo on when to release from cluster union.
	InitData.bCheckConnectivity = bHasAuthority;
	InitData.bGenerateConnectivityEdges = bHasAuthority;

#if CHAOS_DEBUG_NAME
	InitData.DebugName = MakeShared<FString>(FString::Printf(TEXT("%s %s"), *AActor::GetDebugName(GetOwner()), *GetName()));
#endif

	PhysicsProxy = new Chaos::FClusterUnionPhysicsProxy{ this, Parameters, InitData };
	PhysicsProxy->Initialize_External();

	if (FPhysScene_Chaos* Scene = GetChaosScene())
	{
		Scene->AddObject(this, PhysicsProxy);

		if (BodyInstance.bNotifyRigidBodyCollision)
		{
			Scene->RegisterForCollisionEvents(this);
		}
	}
	
	if (bUseClusterUnionAccelerationStructure)
	{
		AccelerationStructure = CreateEmptyAccelerationStructure();
		check(AccelerationStructure != nullptr);
	}

	// It's just logically easier to be consistent on the client to go through the replication route.
	if (IsAuthority())
	{
		for (const FComponentReference& ComponentReference : ClusteredComponentsReferences)
		{
			if (!ComponentReference.OtherActor.IsValid())
			{
				continue;
			}

			AddComponentToCluster(Cast<UPrimitiveComponent>(ComponentReference.GetComponent(ComponentReference.OtherActor.Get())), {});
		}
	}
	else
	{
		// we should not rely on the OnTransformUpdated callback to set the initial position of the external particle
		// if it is not set and the callback does not get called then the cluster union component will be moved to the origin of the world
		const FTransform Transform = GetComponentTransform();
		PhysicsProxy->SetXR_External(Transform.GetLocation(), Transform.GetRotation());

		// Since the initial OnRep_RigidState might've occurred before we actually had a physics state,
		// do OnRep actions again now to make sure we start off with the right data.
		if (bApplyReplicatedRigidStateOnCreatePhysicsState)
		{
			OnRep_RigidState();
		}
	}

	// Cluster union skips UPrimitiveComponent::OnCreatePhysicsState (where the
	// PhysicsReplicationQuantization registration hook lives). Register here, after
	// PhysicsProxy is initialized. GetPhysicsObjectByName(NAME_None) returns null
	// until then so this must come last.
	UE::PhysicsReplicationQuantization::NotifyComponentPhysicsState(this, /*bRegister*/true);
}

void UClusterUnionComponent::OnDestroyPhysicsState()
{
	USceneComponent::OnDestroyPhysicsState();

	// Mirror the OnCreatePhysicsState registration. Safe no op for non replicating actors.
	UE::PhysicsReplicationQuantization::NotifyComponentPhysicsState(this, /*bRegister*/false);

	if (!PhysicsProxy)
	{
		return;
	}

	// We need to make sure we *immediately* disconnect on the GT side since there's no guarantee the normal flow
	// will happen once we've destroyed things.
	for (TPair<TObjectKey<UPrimitiveComponent>, FClusteredComponentData>& ComponentData : PerComponentData)
	{
		HandleRemovedClusteredComponent(ComponentData.Key, ComponentData.Value);
	}
	
	PerComponentData.Reset();

	if (FPhysScene_Chaos* Scene = GetChaosScene())
	{
		Scene->RemoveObject(PhysicsProxy);
	}

	PhysicsProxy = nullptr;
	AccelerationStructure.Reset();
}

void UClusterUnionComponent::OnReceiveReplicatedState(const FVector X, const FQuat R, const FVector V, const FVector W)
{
}

void UClusterUnionComponent::OnUpdateTransform(EUpdateTransformFlags UpdateTransformFlags, ETeleportType Teleport)
{
	USceneComponent::OnUpdateTransform(UpdateTransformFlags, Teleport);

	if (PhysicsProxy && !(UpdateTransformFlags & EUpdateTransformFlags::SkipPhysicsUpdate))
	{
		// If the component transform changes, we need to make sure this update is reflected on the physics thread as well.
		// This code path is generally used when setting the transform manually or when it's set via replication.
		const FTransform Transform = GetComponentTransform();
		PhysicsProxy->SetXR_External(Transform.GetLocation(), Transform.GetRotation());
		if (Teleport == ETeleportType::ResetPhysics)
		{
			PhysicsProxy->SetLinearVelocity_External(FVector::ZeroVector);
			PhysicsProxy->SetAngularVelocity_External(FVector::ZeroVector);
		}
	}
}

FBoxSphereBounds UClusterUnionComponent::CalcBounds(const FTransform& LocalToWorld) const
{
	if (!bHasCachedLocalBounds)
	{
		if (!PhysicsProxy)
		{
			return Super::CalcBounds(LocalToWorld);
		}
		else
		{
			Chaos::FPhysicsObjectHandle Handle = PhysicsProxy->GetPhysicsObjectHandle();
			FLockedReadPhysicsObjectExternalInterface Interface = FPhysicsObjectExternalInterface::LockRead({ &Handle, 1 });
			FBoxSphereBounds NewBounds = Interface->GetBounds({ &Handle, 1 });
			bHasCachedLocalBounds = true;
			CachedLocalBounds = NewBounds;
		}
	}

	return CachedLocalBounds.TransformBy(LocalToWorld);
}

FVector UClusterUnionComponent::GetComponentVelocity() const
{
	if (!PhysicsProxy)
	{
		return Super::GetComponentVelocity();
	}

	Chaos::FPhysicsObjectHandle PhysicsObject = PhysicsProxy->GetPhysicsObjectHandle();
	FLockedReadPhysicsObjectExternalInterface Interface = FPhysicsObjectExternalInterface::LockRead({&PhysicsObject, 1});
	return Interface->GetV(PhysicsObject);
}

bool UClusterUnionComponent::ShouldCreatePhysicsState() const
{
	return true;
}

bool UClusterUnionComponent::HasValidPhysicsState() const
{
	return PhysicsProxy != nullptr;
}

Chaos::FPhysicsObject* UClusterUnionComponent::GetPhysicsObjectById(Chaos::FPhysicsObjectId Id) const
{
	if (!PhysicsProxy)
	{
		return nullptr;
	}
	return PhysicsProxy->GetPhysicsObjectHandle();
}

Chaos::FPhysicsObject* UClusterUnionComponent::GetPhysicsObjectByName(const FName& Name) const
{
	return GetPhysicsObjectById(0);
}

TArray<Chaos::FPhysicsObject*> UClusterUnionComponent::GetAllPhysicsObjects() const
{
	return { GetPhysicsObjectById(0) };
}

Chaos::FPhysicsObjectId UClusterUnionComponent::GetIdFromGTParticle(Chaos::FGeometryParticle* Particle) const
{
	return 0;
}

void UClusterUnionComponent::HandleComponentPhysicsStateChange(UPrimitiveComponent* ChangedComponent, EComponentPhysicsStateChange StateChange)
{
	if (!ChangedComponent || StateChange != EComponentPhysicsStateChange::Created)
	{
		return;
	}

	ChangedComponent->OnComponentPhysicsStateChanged.RemoveDynamic(this, &UClusterUnionComponent::HandleComponentPhysicsStateChange);

	if (FClusterUnionPendingAddData* PendingData = PendingComponentsToAdd.Find(ChangedComponent))
	{
		TArray<int32> AsBoneIDs;
		GetBoneIDsFromComponentData(AsBoneIDs, *PendingData);
		AddComponentToCluster(ChangedComponent, AsBoneIDs);
	}
}

void UClusterUnionComponent::HandleComponentPhysicsStateChangePostAddIntoClusterUnion(UPrimitiveComponent* ChangedComponent, EComponentPhysicsStateChange StateChange)
{
	if (!ChangedComponent || StateChange != EComponentPhysicsStateChange::Destroyed)
	{
		return;
	}

	ChangedComponent->OnComponentPhysicsStateChanged.RemoveDynamic(this, &UClusterUnionComponent::HandleComponentPhysicsStateChangePostAddIntoClusterUnion);
	RemoveComponentFromCluster(ChangedComponent);
}

DECLARE_CYCLE_STAT(TEXT("UClusterUnionComponent::SyncClusterUnionFromProxy"), STAT_ClusterUnionComponent_SyncClusterUnionFromProxy, STATGROUP_Chaos);
void UClusterUnionComponent::SyncClusterUnionFromProxy(const FTransform& NewTransform, TArray<TTuple<UPrimitiveComponent*, FTransform>>* OutNewComponents)
{
	SCOPE_CYCLE_COUNTER(STAT_ClusterUnionComponent_SyncClusterUnionFromProxy);

	// NOTE THAT WE ARE ON THE GAME THREAD HERE.
	if (!PhysicsProxy || !GetWorld())
	{
		return;
	}

	FPhysScene* Scene = GetWorld()->GetPhysicsScene();
	if (!Scene)
	{
		return;
	}

	if (IsAuthority())
	{
		// Create the new replicated state
		const FClusterUnionReplicatedData NewReplicatedRigidState =
		{
			static_cast<uint8>(PhysicsProxy->GetObjectState_External()),
			PhysicsProxy->IsAnchored_External()
		};

		// Only dirty the state if it has changed
		if (ReplicatedRigidState != NewReplicatedRigidState || bDirtyRigidStateOnlyIfChanged == false)
		{
			// Make sure that the new dirty data gets flushed through to clients even if the actor
			// has been made net dormant.
			if (bFlushNetDormancyOnSyncProxy)
			{
				if (AActor* Owner = GetOwner())
				{
					Owner->FlushNetDormancy();
				}
			}

			ReplicatedRigidState = NewReplicatedRigidState;
			MARK_PROPERTY_DIRTY_FROM_NAME(UClusterUnionComponent, ReplicatedRigidState, this);
		}
	}
	
	const Chaos::FClusterUnionSyncedData& FullData = PhysicsProxy->GetSyncedData_External();

	TArray<Chaos::FPBDRigidParticle*> ChildParticles;

	if (FullData.bDidSyncGeometry)
	{
		ChildParticles.Reserve(FullData.ChildParticles.Num());
	}
	
	PerShapeComponentBone.Reset(FullData.ChildParticles.Num());

	// Note that at the UClusterUnionComponent level we really only want to be dealing with components.
	// Hence why we need to modify each of the particles that we synced from the game thread into a
	// component + bone id combination for identification. 
	TMap<FMappedComponentKey, FLocalBonesToTransformMap> MappedData;

	if (bPreAllocateLocalBoneDataMap)
	{
		MappedData.Reserve(PerComponentData.Num());
	}

	{
		FLockedReadPhysicsObjectExternalInterface Interface = FPhysicsObjectExternalInterface::LockRead(GetWorld()->GetPhysicsScene());
		for (const Chaos::FClusterUnionChildData& ChildData : FullData.ChildParticles)
		{
			// Using the scene's proxy to component mapping let's us detect a component physics state was destroyed.
			UPrimitiveComponent* Component = Scene->GetOwningComponent<UPrimitiveComponent>(ChildData.Proxy);
			if (Component && reinterpret_cast<void*>(Component) == ChildData.CachedOwner)
			{
				FMappedComponentKey WrappedComponentKey(Component);
				int32 PrevMappedDataNum = MappedData.Num();
				FLocalBonesToTransformMap& BoneIDToBoneData = MappedData.FindOrAdd(WrappedComponentKey);

				// If we added new entry, and it is for an existing component, preallocate the local map using the
				// existing size as a base
				if (bPreAllocateLocalBoneDataMap && PrevMappedDataNum < MappedData.Num())
				{
					if (FClusteredComponentData* ComponentData = PerComponentData.Find(WrappedComponentKey.ComponentKey))
					{
						if (!ComponentData->BonesData.IsEmpty())
						{
							BoneIDToBoneData.Reserve(ComponentData->BonesData.Num() * LocalBoneDataMapGrowFactor);
						}
					}
				}

				ensure(ChildData.BoneId != INDEX_NONE);
				Chaos::FPhysicsObjectHandle Handle = Component->GetPhysicsObjectById(ChildData.BoneId);
				Chaos::FPBDRigidParticle* Particle = Interface->GetRigidParticle(Handle);

				if (FullData.bDidSyncGeometry)
				{
					ChildParticles.Add(Particle);
				}

				FMappedBoneData MappedBoneData(Handle, Particle, Particle ? Particle->UniqueIdx() : Chaos::FUniqueIdx(), ChildData.ChildToParent);
				BoneIDToBoneData.Add(ChildData.BoneId, MappedBoneData);

				PerShapeComponentBone.Add({Component, ChildData.BoneId});
			}
			else
			{
				if (FullData.bDidSyncGeometry)
				{
					ChildParticles.Add(nullptr);
				}
			
				PerShapeComponentBone.Add({});
			}
		}
	}

	// We don't set the child particles if we did not sync geometry in PullFromPhysicsState because ChildParticles 
	// would not have been populated. However ChildParticles may legitimately be empty if all children were removed
	if (FullData.bDidSyncGeometry)
	{
		PhysicsProxy->ForceSetGeometryChildParticles_External(MoveTemp(ChildParticles));
	}

	// We need to handle any additions, deletions, and modifications to any child in the cluster union here.
	// If a component lives in MappedData but not in PerComponentData, new component!
	// If a component lives in both, then it's a modified component.
	for (const TPair<FMappedComponentKey, FLocalBonesToTransformMap>& Kvp : MappedData)
	{
		HandleAddOrModifiedClusteredComponent(Kvp.Key, Kvp.Value, NewTransform, OutNewComponents);
	}

	// If a component lives in PerComponentData but not in MappedData, deleted component!
	for (TMap<TObjectKey<UPrimitiveComponent>, FClusteredComponentData>::TIterator RemoveComponentDataIterator = PerComponentData.CreateIterator(); RemoveComponentDataIterator; ++RemoveComponentDataIterator)
	{
		// Build a mapped key with a null component ptr as we don't have it at this point and we don't need either
		FMappedComponentKey MappedComponentKey(RemoveComponentDataIterator.Key(), nullptr);
		if (!MappedData.Contains(MappedComponentKey))
		{
			HandleRemovedClusteredComponent(MappedComponentKey.ComponentKey, RemoveComponentDataIterator.Value());

			RemoveComponentDataIterator.RemoveCurrent();
		}
	}

	const FBoxSphereBounds OldBounds = CachedLocalBounds;
	bHasCachedLocalBounds = false;
	UpdateBounds();

	// Check if the old bounds is approximately equal to the new bounds. If so, we don't need to send an event
	// saying that the cluster union changed in shape/position.
	const bool bIsSameBounds = OldBounds.Origin.Equals(CachedLocalBounds.Origin) && OldBounds.BoxExtent.Equals(CachedLocalBounds.BoxExtent) && (FMath::Abs(OldBounds.SphereRadius - CachedLocalBounds.SphereRadius) < UE_KINDA_SMALL_NUMBER);
	if (!bIsSameBounds)
	{
		OnComponentBoundsChangedEvent.Broadcast(this, CachedLocalBounds);
	}

	if (OnClusterUnionPostSyncBodiesEvent.IsBound())
	{
		TArray<TObjectPtr<UPrimitiveComponent>> PrimitiveComponents;
		PrimitiveComponents.Reserve(PerComponentData.Num());

		for (auto Iter = PerComponentData.CreateIterator(); Iter; ++Iter)
		{
			PrimitiveComponents.Add(Iter.Key().ResolveObjectPtr());
		}
		OnClusterUnionPostSyncBodiesEvent.Broadcast({ this,PrimitiveComponents });
	}
}

void UClusterUnionComponent::HandleAddOrModifiedClusteredComponent(const FMappedComponentKey& ChangedComponentData, const FLocalBonesToTransformMap& PerBoneChildToParent, const FTransform& NewTransform, TArray<TTuple<UPrimitiveComponent*, FTransform>>* OutNewComponents)
{
	if (!ChangedComponentData.ComponentPtr || !ChangedComponentData.ComponentPtr->HasValidPhysicsState() || !ChangedComponentData.ComponentPtr->GetWorld())
	{
		return;
	}

	const bool bIsNew = !PerComponentData.Contains(ChangedComponentData.ComponentKey);
	FClusteredComponentData& ComponentData = PerComponentData.FindOrAdd(ChangedComponentData.ComponentKey);

	// If this is a *new* component that we're keeping track of then there's additional book-keeping
	// we need to do to make sure we don't forget what exactly we're tracking. Additionally, we need to
	// modify the component and its parent actor to ensure their replication stops.
	if (bIsNew)
	{
		PendingComponentSync.Remove(ChangedComponentData.ComponentKey);

		// Force the component and its parent actor to stop replicating movement.
		// Setting the component to not replicate should be sufficient since a simulating
		// component shouldn't be doing much more than replicating its position anyway.
		if (AActor* Owner = ChangedComponentData.ComponentPtr->GetOwner())
		{
			if (FClusteredActorData* Data = ActorToComponents.Find(Owner))
			{
				Data->Components.Add(ChangedComponentData.ComponentKey);
			}
			else
			{
				FClusteredActorData NewData;
				NewData.Components.Add(ChangedComponentData.ComponentKey);
				NewData.bWasReplicatingMovement = Owner->IsReplicatingMovement();
				ActorToComponents.Add(Owner, NewData);

				if (IsAuthority() && Owner != GetOwner())
				{
					Owner->SetReplicatingMovement(false);
				}
			}

			ComponentData.Owner = Owner;
		}

		ComponentData.bWasReplicating = ChangedComponentData.ComponentPtr->GetIsReplicated();
		if (IsAuthority())
		{
			if (AActor* Owner = ChangedComponentData.ComponentPtr->GetOwner())
			{
				// Create a replicated proxy component and add it to the actor being added to the cluster.
				// This component will take care of replicating this addition into the cluster.
				TObjectPtr<UClusterUnionReplicatedProxyComponent> ReplicatedProxy = NewObject<UClusterUnionReplicatedProxyComponent>(Owner);
				if (ensure(ReplicatedProxy))
				{
					ReplicatedProxy->RegisterComponent();
					ReplicatedProxy->SetParentClusterUnion(this);
					ReplicatedProxy->SetChildClusteredComponent(ChangedComponentData.ComponentPtr);
					ReplicatedProxy->SetIsReplicated(true);
				}

				ComponentData.ReplicatedProxyComponent = ReplicatedProxy;
			}
		}
	}

	TArray<FClusterUnionBoneData> RemovedBoneIDs;
	RemovedBoneIDs.Reserve(ComponentData.BonesData.Num());

	for (const FClusterUnionBoneData& BoneData : ComponentData.BonesData)
	{
		if (!PerBoneChildToParent.Contains(BoneData.ID))
		{
			// In the case where we need to keep the acceleration structure up to date, we need to make sure old bone ids are properly
			// removed from the acceleration structure.
			if (AccelerationStructure)
			{
				FExternalSpatialAccelerationPayload Handle;
				Handle.Initialize(ChangedComponentData.ComponentKey, BoneData.ID, BoneData.ParticleID);
				if (ensure(Handle.IsValid()))
				{
					AccelerationStructure->RemoveElement(Handle);
				}
			}

			RemovedBoneIDs.Emplace(FClusterUnionBoneData(BoneData.ID, BoneData.ParticleID));
		}
	}

	ComponentData.BonesData.Reset();
	ComponentData.BonesData.Reserve(PerBoneChildToParent.Num());

	Algo::Transform(PerBoneChildToParent, ComponentData.BonesData, [](const TPair<int32, FMappedBoneData>& MappedData)
	{
		return FClusterUnionBoneData{MappedData.Key, MappedData.Value.ParticleID };
	});

	BroadcastComponentAddedEvents(ChangedComponentData.ComponentPtr, ComponentData.BonesData, bIsNew, RemovedBoneIDs);

	if (IsAuthority() && ComponentData.ReplicatedProxyComponent.IsValid())
	{
		// We really only need to do modifications on the server since that's where we're changing the replicated proxy to broadcast this data change.
		TObjectPtr<UClusterUnionReplicatedProxyComponent> ReplicatedProxy = ComponentData.ReplicatedProxyComponent.Get();
		
		TArray<int32> AsBoneIdsArray;
		GetBoneIDsFromComponentData(AsBoneIdsArray, ComponentData);
		ReplicatedProxy->SetParticleBoneIds(AsBoneIdsArray);

		for (const TPair<int32, FMappedBoneData>& Kvp : PerBoneChildToParent)
		{
			ReplicatedProxy->SetParticleChildToParent(Kvp.Key, Kvp.Value.ChildToParentTransform);
		}
	}

	OnChildToParentUpdated(ChangedComponentData.ComponentPtr, PerBoneChildToParent, NewTransform, OutNewComponents);

	if (PerBoneChildToParent.Num())
	{
		// get the interface from the first element 
		FLockedReadPhysicsObjectExternalInterface Interface = FPhysicsObjectExternalInterface::LockRead(GetWorld()->GetPhysicsScene());

		// One more loop to ensure that our sets of physics objects are valid and up to date.
		// This needs to happen on both the client and the server.
		for (const TPair<int32, FMappedBoneData>& Kvp : PerBoneChildToParent)
		{
			if (AccelerationStructure)
			{
				if (Chaos::FPhysicsObjectHandle PhysicsObject = Kvp.Value.PhysicsObjectHandle)
				{
					const FBox HandleBounds = GetWorldBoundsForParticle(Kvp.Value.RigidParticle);
					FExternalSpatialAccelerationPayload Handle;
					Handle.Initialize(ChangedComponentData.ComponentKey, Kvp.Key, Kvp.Value.ParticleID);
					if (ensure(Handle.IsValid()))
					{
						AccelerationStructure->UpdateElement(Handle, Chaos::TAABB<Chaos::FReal, 3>{HandleBounds.Min, HandleBounds.Max}, HandleBounds.IsValid != 0);
					}
				}
			}
		}
	}
}

static UClusterUnionReplicatedProxyComponent* GetReplicatedProxyFromComponent(const UClusterUnionComponent* ClusterUnionComponent, const UPrimitiveComponent* Component)
{
	if (Component)
	{
		if (AActor* ComponentOwner = Component->GetOwner())
		{
			TInlineComponentArray<UClusterUnionReplicatedProxyComponent*> ReplicatedProxyComponents(ComponentOwner);
			for (UClusterUnionReplicatedProxyComponent* ReplicatedProxyComponent : ReplicatedProxyComponents)
			{
				if (ReplicatedProxyComponent->GetParentClusterUnionComponent() == ClusterUnionComponent && ReplicatedProxyComponent->GetChildClusteredComponent() == Component)
				{
					return ReplicatedProxyComponent;
				}
			}
		}
	}
	return nullptr;
}

void UClusterUnionComponent::HandleRemovedClusteredComponent(TObjectKey<UPrimitiveComponent> RemovedComponent, const FClusteredComponentData& ComponentData)
{
	if (AccelerationStructure)
	{
		for (const FClusterUnionBoneData& BoneData : ComponentData.BonesData)
		{
			FExternalSpatialAccelerationPayload Handle;
			Handle.Initialize(RemovedComponent, BoneData.ID, BoneData.ParticleID);

			if (ensure(Handle.IsValid()))
			{
				AccelerationStructure->RemoveElement(Handle);
			}
		}
	}

	UPrimitiveComponent* RemovedComponentPtr = RemovedComponent.ResolveObjectPtr();

	if (IsAuthority())
	{
		if (UClusterUnionReplicatedProxyComponent* ProxyComponent = ComponentData.ReplicatedProxyComponent.Get())
		{
			ProxyComponent->DestroyComponent();
		}
	}
	else
	{
		// On the client, we need to reset the state of the replicated proxy has it may outlive the cluster union
		// and may still be in existence when the cluster union is created again 
		// Note: we need to get the replicated component from the actor on the client because the PerComponentData do not set the replicated proxy pointer (unlike the server )
		if (UClusterUnionReplicatedProxyComponent* ProxyComponent = GetReplicatedProxyFromComponent(this, RemovedComponentPtr))
		{
			ProxyComponent->ResetTransientState();
		}
	}

	// Need to get even if pending kill just in case we received a removal because the actor got destroyed!
	TObjectKey<AActor> OwnerKey { ComponentData.Owner.Get(true) };
	if (FClusteredActorData* ActorData = ActorToComponents.Find(OwnerKey))
	{
		ActorData->Components.Remove(RemovedComponent);

		if (ActorData->Components.IsEmpty())
		{
			if (AActor* Owner = ComponentData.Owner.Get())
			{
				if (IsAuthority())
				{
					Owner->SetReplicatingMovement(ActorData->bWasReplicatingMovement);
				}
			}

			ActorToComponents.Remove(OwnerKey);
		}
	}

	if (RemovedComponentPtr)
	{
		BroadcastComponentRemovedEvents(RemovedComponentPtr, ComponentData.BonesData);
	}
	PendingComponentsToAdd.Remove(RemovedComponent);
	PendingComponentSync.Remove(RemovedComponent);
}

void UClusterUnionComponent::BroadcastComponentAddedEvents(UPrimitiveComponent* ChangedComponent, const TArray<FClusterUnionBoneData>& BoneIds, bool bIsNew, const TArray<FClusterUnionBoneData>& RemovedBoneIDs)
{
	// TODO: Add a new delegate that takes a TArray and deprecate this one
	if (OnComponentAddedEvent.IsBound())
	{
		TSet<int32> BoneIDsSet;
		Algo::Transform(BoneIds, BoneIDsSet, &FClusterUnionBoneData::ID);
		OnComponentAddedEvent.Broadcast(ChangedComponent, BoneIDsSet, bIsNew);
	}
	if (OnComponentAddedNativeEvent.IsBound())
	{
		OnComponentAddedNativeEvent.Broadcast(ChangedComponent, BoneIds, RemovedBoneIDs, bIsNew);
	}
}

void UClusterUnionComponent::BroadcastComponentRemovedEvents(UPrimitiveComponent* ChangedComponent, const TArray<FClusterUnionBoneData>& InRemovedBonesData)
{
	if (OnComponentRemovedEvent.IsBound())
	{
		OnComponentRemovedEvent.Broadcast(ChangedComponent);
	}
	if (OnComponentRemovedNativeEvent.IsBound())
	{
		OnComponentRemovedNativeEvent.Broadcast(ChangedComponent, InRemovedBonesData);
	}
}

void UClusterUnionComponent::OnRep_RigidState()
{
	// Prior to having a proxy we can't set its state.
	// Also, if our state is 0 we have not yet receive a valid object state from
	// the server (we call this onrep in OnCreatePhysicsState too incase we got a
	// new state but didn't have a proxy at that time)
	if (!PhysicsProxy || (ReplicatedRigidState.ObjectState == 0 && GSkipZeroStateInOnRep))
	{
		return;
	}

	Chaos::EObjectStateType ObjectState = static_cast<Chaos::EObjectStateType>(ReplicatedRigidState.ObjectState);

	// If Physics Prediction is enabled, only update to and from Kinematic state (sleeping/dynamic switching will be handled via the enabled EPhysicsReplicationMode)
	if (!bPhysicsPredictionEnabled 
		|| PhysicsProxy->GetObjectState_External() == Chaos::EObjectStateType::Kinematic
		|| ObjectState == Chaos::EObjectStateType::Kinematic)
	{
		SetRigidState(ObjectState);
	}
}

void UClusterUnionComponent::GetLifetimeReplicatedProps(TArray<FLifetimeProperty>& OutLifetimeProps) const
{
	UPrimitiveComponent::GetLifetimeReplicatedProps(OutLifetimeProps);

	FDoRepLifetimeParams Params;
	Params.bIsPushBased = true;

	DOREPLIFETIME_WITH_PARAMS_FAST(UClusterUnionComponent, ReplicatedRigidState, Params);

	// Allow the physical parameters of this component to be replicated via
	// physicsreplication even if component replication is enabled
	DISABLE_REPLICATED_PROPERTY_FAST(USceneComponent, RelativeLocation);
	DISABLE_REPLICATED_PROPERTY_FAST(USceneComponent, RelativeRotation);
}

void UClusterUnionComponent::ForceSetChildToParent(UPrimitiveComponent* InComponent, const TArray<int32>& BoneIds, const TArray<FTransform>& ChildToParent)
{
	if (!PhysicsProxy || !ensure(InComponent) || !ensure(BoneIds.Num() == ChildToParent.Num()))
	{
		return;
	}

	TArray< Chaos::FPhysicsObjectHandle> Objects;
	Objects.Reserve(BoneIds.Num());

	for (int32 Index = 0; Index < BoneIds.Num(); ++Index)
	{
		Chaos::FPhysicsObjectHandle Handle = InComponent->GetPhysicsObjectById(BoneIds[Index]);
		const bool bIsValidHandle = ensureMsgf(Handle != nullptr, TEXT("UClusterUnionComponent::ForceSetChildToParent invalid BoneID %d on %s"), BoneIds[Index], *InComponent->GetFullName());

		if (bIsValidHandle)
		{
			Objects.Add(Handle);
		}
	}

	// Need to lock the ChildToParent to prevent the PT from trying to compute it.
	if (!Objects.IsEmpty())
	{
		PhysicsProxy->BulkSetChildToParent_External(Objects, ChildToParent, true);
	}
}

void UClusterUnionComponent::SetDisabled(const bool Disabled)
{
	if (PhysicsProxy)
	{
		PhysicsProxy->SetDisabled_External(Disabled);
	}
}

bool UClusterUnionComponent::GetDisabled() const
{
	if (PhysicsProxy)
	{
		return PhysicsProxy->GetDisabled_External();
	}

	return false;
}

void UClusterUnionComponent::SetGravityEnabled(const bool Enabled)
{
	if (PhysicsProxy)
	{
		PhysicsProxy->SetGravityEnabled_External(Enabled);
	}
}

bool UClusterUnionComponent::GetGravityEnabled() const
{
	if (PhysicsProxy)
	{
		return PhysicsProxy->GetGravityEnabled_External();
	}

	return false;
}

void UClusterUnionComponent::SetSleepType(const Chaos::ESleepType SleepType)
{
	if (PhysicsProxy)
	{
		PhysicsProxy->SetSleepType_External(SleepType);
	}
}

Chaos::ESleepType UClusterUnionComponent::GetSleepType() const
{
	if (PhysicsProxy)
	{
		return PhysicsProxy->GetSleepType_External();
	}

	return Chaos::ESleepType::MaterialSleep;
}
	
void UClusterUnionComponent::SetRigidState(const Chaos::EObjectStateType ObjectState)
{
	if (PhysicsProxy)
	{
		PhysicsProxy->SetObjectState_External(ObjectState);
		const bool bIsDynamic = (ObjectState == Chaos::EObjectStateType::Dynamic || ObjectState == Chaos::EObjectStateType::Sleeping);
		PhysicsProxy->SetIsAnchored_External(!bIsDynamic);
	}
}

Chaos::EObjectStateType UClusterUnionComponent::GetRigidState() const
{
	if (PhysicsProxy)
	{
		return PhysicsProxy->GetObjectState_External();
	}

	return Chaos::EObjectStateType::Uninitialized;
}

void UClusterUnionComponent::SetSimulatePhysics(bool bSimulate)
{
	SetRigidState(bSimulate ? Chaos::EObjectStateType::Dynamic : Chaos::EObjectStateType::Kinematic);
}

void UClusterUnionComponent::WakeAllRigidBodies()
{
	if (PhysicsProxy)
	{
		PhysicsProxy->Wake_External();
	}
}

bool UClusterUnionComponent::IsAnyRigidBodyAwake()
{
	TArray<Chaos::FPhysicsObject*> PhysicsObjects = GetAllPhysicsObjects();
	FLockedReadPhysicsObjectExternalInterface Interface = FPhysicsObjectExternalInterface::LockRead(PhysicsObjects);

	return !Interface->AreAllSleeping(PhysicsObjects);
}

void UClusterUnionComponent::SetMassOverrideInKg(FName BoneName, float MassInKg, bool bOverrideMass)
{
	if (PhysicsProxy)
	{
		PhysicsProxy->SetMass_External(MassInKg);
	}
}

DECLARE_CYCLE_STAT(TEXT("UClusterUnionComponent::LineTraceComponentMulti"), STAT_ClusterUnionComponentLineTraceComponentMulti, STATGROUP_Chaos);
bool UClusterUnionComponent::LineTraceComponent(TArray<FHitResult>& OutHit, const FVector Start, const FVector End, ECollisionChannel TraceChannel, const struct FCollisionQueryParams& Params, const struct FCollisionResponseParams& ResponseParams, const struct FCollisionObjectQueryParams& ObjectParams)
{
	SCOPE_CYCLE_COUNTER(STAT_ClusterUnionComponentLineTraceComponentMulti);
	OutHit.Reset();
	if (AccelerationStructure)
	{
		FGenericRaycastPhysicsInterfaceUsingSpatialAcceleration<IExternalSpatialAcceleration>::RaycastMulti(*AccelerationStructure, GetWorld(), OutHit, Start, End, TraceChannel, Params, ResponseParams, ObjectParams);
		return !OutHit.IsEmpty();
	}

	VisitAllCurrentChildComponentsForCollision(TraceChannel, Params, ResponseParams, ObjectParams,
		[&Start, &End, TraceChannel, &Params, &ResponseParams, &ObjectParams, &OutHit](UPrimitiveComponent* Component)
		{
			FHitResult ComponentHit;
			if (Component->LineTraceComponent(ComponentHit, Start, End, TraceChannel, Params, ResponseParams, ObjectParams))
			{
				OutHit.Add(ComponentHit);
			}

			return true;
		}
	);

	return !OutHit.IsEmpty();
}

DECLARE_CYCLE_STAT(TEXT("UClusterUnionComponent::SweepComponentMulti"), STAT_ClusterUnionComponentSweepComponentMulti, STATGROUP_Chaos);
bool UClusterUnionComponent::SweepComponent(TArray<FHitResult>& OutHit, const FVector Start, const FVector End, const FQuat& ShapeWorldRotation, const FPhysicsGeometry& Geometry, ECollisionChannel TraceChannel, const struct FCollisionQueryParams& Params, const struct FCollisionResponseParams& ResponseParams, const struct FCollisionObjectQueryParams& ObjectParams)
{
	SCOPE_CYCLE_COUNTER(STAT_ClusterUnionComponentSweepComponentMulti);
	OutHit.Reset();
	if (AccelerationStructure)
	{
		FGenericGeomPhysicsInterfaceUsingSpatialAcceleration<IExternalSpatialAcceleration, FPhysicsGeometry>::GeomSweepMulti(*AccelerationStructure, GetWorld(), Geometry, ShapeWorldRotation, OutHit, Start, End, TraceChannel, Params, ResponseParams, ObjectParams);
		return !OutHit.IsEmpty();
	}

	VisitAllCurrentChildComponentsForCollision(TraceChannel, Params, ResponseParams, ObjectParams,
		[&Geometry, &Start, &End, &ShapeWorldRotation, TraceChannel, &Params, &ResponseParams, &ObjectParams, &OutHit](UPrimitiveComponent* Component)
		{
			FHitResult ComponentHit;
			if (Component->SweepComponent(ComponentHit, Start, End, ShapeWorldRotation, Geometry, TraceChannel, Params, ResponseParams, ObjectParams))
			{
				OutHit.Add(ComponentHit);
			}

			return true;
		}
	);

	return !OutHit.IsEmpty();
}

DECLARE_CYCLE_STAT(TEXT("UClusterUnionComponent::LineTraceComponentSingle"), STAT_ClusterUnionComponentLineTraceComponentSingle, STATGROUP_Chaos);
bool UClusterUnionComponent::LineTraceComponent(FHitResult& OutHit, const FVector Start, const FVector End, ECollisionChannel TraceChannel, const struct FCollisionQueryParams& Params, const struct FCollisionResponseParams& ResponseParams, const struct FCollisionObjectQueryParams& ObjectParams)
{
	SCOPE_CYCLE_COUNTER(STAT_ClusterUnionComponentLineTraceComponentSingle);

	OutHit.Distance = TNumericLimits<float>::Max();
	if (AccelerationStructure)
	{
		FGenericRaycastPhysicsInterfaceUsingSpatialAcceleration<IExternalSpatialAcceleration>::RaycastSingle(*AccelerationStructure, GetWorld(), OutHit, Start, End, TraceChannel, Params, ResponseParams, ObjectParams);
		return OutHit.HasValidHitObjectHandle();
	}

	bool bHasHit = false;

	VisitAllCurrentChildComponentsForCollision(TraceChannel, Params, ResponseParams, ObjectParams,
		[&Start, &End, TraceChannel, &Params, &ResponseParams, &ObjectParams, &bHasHit, &OutHit](UPrimitiveComponent* Component)
		{
			FHitResult ComponentHit;
			if (Component->LineTraceComponent(ComponentHit, Start, End, TraceChannel, Params, ResponseParams, ObjectParams) && ComponentHit.Distance < OutHit.Distance)
			{
				bHasHit = true;
				OutHit = ComponentHit;
			}

			return true;
		}
	);

	return bHasHit;
}

DECLARE_CYCLE_STAT(TEXT("UClusterUnionComponent::SweepComponentSingle"), STAT_ClusterUnionComponentSweepComponentSingle, STATGROUP_Chaos);
bool UClusterUnionComponent::SweepComponent(FHitResult& OutHit, const FVector Start, const FVector End, const FQuat& ShapeWorldRotation, const FPhysicsGeometry& Geometry, ECollisionChannel TraceChannel, const struct FCollisionQueryParams& Params, const struct FCollisionResponseParams& ResponseParams, const struct FCollisionObjectQueryParams& ObjectParams)
{
	SCOPE_CYCLE_COUNTER(STAT_ClusterUnionComponentSweepComponentSingle);
	OutHit.Distance = TNumericLimits<float>::Max();
	if (AccelerationStructure)
	{
		FGenericGeomPhysicsInterfaceUsingSpatialAcceleration<IExternalSpatialAcceleration, FPhysicsGeometry>::GeomSweepSingle(*AccelerationStructure, GetWorld(), Geometry, ShapeWorldRotation, OutHit, Start, End, TraceChannel, Params, ResponseParams, ObjectParams);
		return OutHit.HasValidHitObjectHandle();
	}

	bool bHasHit = false;

	VisitAllCurrentChildComponentsForCollision(TraceChannel, Params, ResponseParams, ObjectParams,
		[&Geometry, &Start, &End, &ShapeWorldRotation, TraceChannel, &Params, &ResponseParams, &ObjectParams, &bHasHit, &OutHit](UPrimitiveComponent* Component)
		{
			FHitResult ComponentHit;
			if (Component->SweepComponent(ComponentHit, Start, End, ShapeWorldRotation, Geometry, TraceChannel, Params, ResponseParams, ObjectParams) && ComponentHit.Distance < OutHit.Distance)
			{
				bHasHit = true;
				OutHit = ComponentHit;
			}

			return true;
		}
	);

	return bHasHit;
}

DECLARE_CYCLE_STAT(TEXT("UClusterUnionComponent::OverlapComponentWithResult"), STAT_ClusterUnionComponentOverlapComponentWithResult, STATGROUP_Chaos);
bool UClusterUnionComponent::OverlapComponentWithResult(const FVector& Pos, const FQuat& Rot, const FPhysicsGeometry& Geometry, ECollisionChannel TraceChannel, const struct FCollisionQueryParams& Params, const struct FCollisionResponseParams& ResponseParams, const struct FCollisionObjectQueryParams& ObjectParams, TArray<FOverlapResult>& OutOverlap) const
{
	SCOPE_CYCLE_COUNTER(STAT_ClusterUnionComponentOverlapComponentWithResult);
	if (AccelerationStructure)
	{
		FGenericGeomPhysicsInterfaceUsingSpatialAcceleration<IExternalSpatialAcceleration, FPhysicsGeometry>::GeomOverlapMulti(*AccelerationStructure, GetWorld(), Geometry, Pos, Rot, OutOverlap, TraceChannel, Params, ResponseParams, ObjectParams);
		return !OutOverlap.IsEmpty();
	}

	bool bHasOverlap = false;

	FVector QueryHalfExtent = Geometry.BoundingBox().Extents();
	FBoxSphereBounds QueryBounds(FBox(-QueryHalfExtent, QueryHalfExtent));
	QueryBounds = QueryBounds.TransformBy(FTransform(Rot, Pos));

	VisitAllCurrentChildComponentsForCollision(TraceChannel, Params, ResponseParams, ObjectParams,
		[&QueryBounds, &Geometry, &Pos, &Rot, TraceChannel, &Params, &ResponseParams, &ObjectParams, &bHasOverlap, &OutOverlap](UPrimitiveComponent* Component)
		{
			if (FBoxSphereBounds::SpheresIntersect(QueryBounds, Component->Bounds))
			{
				TArray<FOverlapResult> SubOverlaps;
				if (Component->OverlapComponentWithResult(Pos, Rot, Geometry, TraceChannel, Params, ResponseParams, ObjectParams, SubOverlaps))
				{
					bHasOverlap = true;
					OutOverlap.Append(SubOverlaps);
				}
			}

			return true;
		}
	);

	return bHasOverlap;
}

DECLARE_CYCLE_STAT(TEXT("UClusterUnionComponent::ComponentOverlapComponentWithResultImpl"), STAT_ClusterUnionComponentComponentOverlapComponentWithResultImpl, STATGROUP_Chaos);
bool UClusterUnionComponent::ComponentOverlapComponentWithResultImpl(const class UPrimitiveComponent* const PrimComp, const FVector& Pos, const FQuat& Rot, const FCollisionQueryParams& Params, TArray<FOverlapResult>& OutOverlap) const
{
	SCOPE_CYCLE_COUNTER(STAT_ClusterUnionComponentComponentOverlapComponentWithResultImpl);
	if(!PrimComp)
	{
		return false;
	}

	bool bHasOverlap = false;
	if (AccelerationStructure)
	{
		TArray<Chaos::FPhysicsObjectHandle> InObjects = PrimComp->GetAllPhysicsObjects();
		FLockedReadPhysicsObjectExternalInterface Interface = FPhysicsObjectExternalInterface::LockRead(InObjects);
		InObjects = InObjects.FilterByPredicate(
			[&Interface](Chaos::FPhysicsObjectHandle Handle)
			{
				return !Interface->AreAllDisabled({ &Handle, 1 });
			}
		);

		Interface->VisitEveryShape(InObjects,
			[this, &bHasOverlap, &Pos, &Rot, &OutOverlap, &Params](const Chaos::FConstPhysicsObjectHandle Handle, Chaos::FShapeInstanceProxy* Shape)
			{
				if (Shape)
				{
					if (Chaos::FImplicitObjectPtr Geometry = Shape->GetGeometry())
					{
						bHasOverlap |= FGenericGeomPhysicsInterfaceUsingSpatialAcceleration<IExternalSpatialAcceleration, FPhysicsGeometry>::GeomOverlapMulti(*AccelerationStructure, GetWorld(), *Geometry, Pos, Rot, OutOverlap, DefaultCollisionChannel, Params, FCollisionResponseParams::DefaultResponseParam, FCollisionObjectQueryParams::DefaultObjectQueryParam);
					}
				}

				return true;
			});

		return bHasOverlap;
	}

	FBoxSphereBounds QueryBounds = PrimComp->Bounds.TransformBy(FTransform(Rot, Pos));

	VisitAllCurrentChildComponentsForCollision(DefaultCollisionChannel, Params, FCollisionResponseParams::DefaultResponseParam, FCollisionObjectQueryParams::DefaultObjectQueryParam,
		[&QueryBounds, PrimComp, &Pos, &Rot, &Params, &bHasOverlap, &OutOverlap](UPrimitiveComponent* Component)
		{
			if (FBoxSphereBounds::SpheresIntersect(QueryBounds, Component->Bounds))
			{
				TArray<FOverlapResult> SubOverlaps;
				if (Component->ComponentOverlapComponentWithResult(PrimComp, Pos, Rot, Params, SubOverlaps))
				{
					bHasOverlap = true;
					OutOverlap.Append(SubOverlaps);
				}
			}

			return true;
		}
	);

	return bHasOverlap;
}

TArray<UPrimitiveComponent*> UClusterUnionComponent::GetAllCurrentChildComponents() const
{
	TArray<UPrimitiveComponent*> Components;
	Components.Reserve(PerComponentData.Num() + PendingComponentSync.Num());

	VisitAllCurrentChildComponents(
		[&Components](UPrimitiveComponent* Component)
		{
			Components.Add(Component);
			return true;
		}
	);

	return Components;
}

TArray<AActor*> UClusterUnionComponent::GetAllCurrentActors() const
{
	TArray<AActor*> Actors;
	Actors.Reserve(ActorToComponents.Num());

	VisitAllCurrentActors(
		[&Actors](AActor* Actor)
		{
			Actors.Add(Actor);
			return true;
		}
	);

	return Actors;
}

void UClusterUnionComponent::VisitAllCurrentChildComponents(const TFunction<bool(UPrimitiveComponent*)>& Lambda) const
{
	for (const TPair<TObjectKey<UPrimitiveComponent>, FClusterUnionPendingAddData>& Kvp : PendingComponentSync)
	{
		if (UPrimitiveComponent* Component = Kvp.Key.ResolveObjectPtr(); Component && Component->HasValidPhysicsState())
		{
			if (!Lambda(Component))
			{
				return;
			}
		}
	}

	for (const TPair<TObjectKey<UPrimitiveComponent>, FClusteredComponentData>& Kvp : PerComponentData)
	{
		if (UPrimitiveComponent* Component = Kvp.Key.ResolveObjectPtr(); Component && Component->HasValidPhysicsState() && !Kvp.Value.bPendingDeletion)
		{
			if (!Lambda(Component))
			{
				return;
			}
		}
	}
}

void UClusterUnionComponent::VisitAllCurrentActors(const TFunction<bool(AActor*)>& Lambda) const
{
	for (const TPair<TObjectKey<AActor>, FClusteredActorData>& Kvp : ActorToComponents)
	{
		if (AActor* Actor = Kvp.Key.ResolveObjectPtr())
		{
			if (!Lambda(Actor))
			{
				return;
			}
		}
	}
}

void UClusterUnionComponent::VisitAllCurrentChildComponentsForCollision(ECollisionChannel TraceChannel, const struct FCollisionQueryParams& Params, const struct FCollisionResponseParams& ResponseParams, const struct FCollisionObjectQueryParams& ObjectParams, const TFunction<bool(UPrimitiveComponent*)>& Lambda) const
{
	TSet<uint32> IgnoredActors;
	IgnoredActors.Reserve(Params.GetIgnoredSourceObjects().Num());
	for (uint32 Id : Params.GetIgnoredSourceObjects())
	{
		IgnoredActors.Add(Id);
	}

	TSet<uint32> IgnoredComponents;
	IgnoredComponents.Reserve(Params.GetIgnoredComponents().Num());
	for (uint32 Id : Params.GetIgnoredComponents())
	{
		IgnoredComponents.Add(Id);
	}

	VisitAllCurrentChildComponents(
		[TraceChannel, &IgnoredActors, &IgnoredComponents, &Lambda](UPrimitiveComponent* Component)
		{
			if (Component)
			{
				// TODO: Support all the other filters that could exist here.
				if (IgnoredComponents.Contains(Component->GetUniqueID()))
				{
					return true;
				}

				if (AActor* Owner = Component->GetOwner())
				{
					if (IgnoredActors.Contains(Owner->GetUniqueID()))
					{
						return true;
					}
				}

				if (Component->GetCollisionResponseToChannel(TraceChannel) == ECollisionResponse::ECR_Ignore)
				{
					return true;
				}

				Lambda(Component);
			}
			return true;
		}
	);
}

TArray<int32> UClusterUnionComponent::GetAddedBoneIdsForComponent(UPrimitiveComponent* Component) const
{
	if (const FClusteredComponentData* Data = PerComponentData.Find(Component))
	{
		TArray<int32> BoneIDsArray;
		GetBoneIDsFromComponentData(BoneIDsArray, *Data);

		return MoveTemp(BoneIDsArray);
	}

	if (const FClusterUnionPendingAddData* Data = PendingComponentSync.Find(Component))
	{
		TArray<int32> BoneIDsArray;
		GetBoneIDsFromComponentData(BoneIDsArray, *Data);

		return MoveTemp(BoneIDsArray);
	}

	return {};
}

void UClusterUnionComponent::AddReferencedObjects(UObject* InThis, FReferenceCollector& Collector)
{
	UPrimitiveComponent::AddReferencedObjects(InThis, Collector);

	UClusterUnionComponent* This = CastChecked<UClusterUnionComponent>(InThis);

	{
		const UScriptStruct* ScriptStruct = FClusteredComponentData::StaticStruct();
		for (TPair<TObjectKey<UPrimitiveComponent>, FClusteredComponentData>& Kvp : This->PerComponentData)
		{
			TWeakObjectPtr<const UScriptStruct> ScriptStructPtr{ScriptStruct};
			Collector.AddReferencedObjects(ScriptStructPtr, reinterpret_cast<void*>(&Kvp.Value), This, nullptr);
		}
	}

	{
		const UScriptStruct* ScriptStruct = FClusteredActorData::StaticStruct();
		for (TPair<TObjectKey<AActor>, FClusteredActorData>& Kvp : This->ActorToComponents)
		{
			TWeakObjectPtr<const UScriptStruct> ScriptStructPtr{ScriptStruct};
			Collector.AddReferencedObjects(ScriptStructPtr, reinterpret_cast<void*>(&Kvp.Value), This, nullptr);
		}
	}

	{
		const UScriptStruct* ScriptStruct = FClusterUnionPendingAddData::StaticStruct();
		for (TPair<TObjectKey<UPrimitiveComponent>, FClusterUnionPendingAddData>& Kvp : This->PendingComponentsToAdd)
		{
			TWeakObjectPtr<const UScriptStruct> ScriptStructPtr{ScriptStruct};			
			Collector.AddReferencedObjects(ScriptStructPtr, reinterpret_cast<void*>(&Kvp.Value), This, nullptr);
		}
	}

	{
		const UScriptStruct* ScriptStruct = FClusterUnionPendingAddData::StaticStruct();
		for (TPair<TObjectKey<UPrimitiveComponent>, FClusterUnionPendingAddData>& Kvp : This->PendingComponentSync)
		{
			Collector.AddReferencedObjects(ObjectPtrWrap(ScriptStruct), reinterpret_cast<void*>(&Kvp.Value), This, nullptr);
		}
	}
}

int32 UClusterUnionComponent::NumChildClusterComponents() const
{
	return PendingComponentSync.Num() + PerComponentData.Num();
}

bool UClusterUnionComponent::DoCustomNavigableGeometryExport(FNavigableGeometryExport& GeomExport) const
{
	bool bHasData = false;
	VisitAllCurrentChildComponents(
		[&bHasData, &GeomExport](UPrimitiveComponent* Component)
		{
			if (Component)
			{
				bHasData |= Component->DoCustomNavigableGeometryExport(GeomExport);
			}
			return true;
		}
	);
	return bHasData;
}

void UClusterUnionComponent::ChangeIfComponentBonesAreMainParticle(UPrimitiveComponent* Component, const TArray<int32>& BoneIds, bool bIsMain)
{
	if (!PhysicsProxy || BoneIds.IsEmpty() || !Component)
	{
		return;
	}

	TArray<Chaos::FPhysicsObjectHandle> PhysicsObjects = GetAllPhysicsObjectsById(Component, BoneIds);
	if (PhysicsObjects.IsEmpty())
	{
		return;
	}

	PhysicsProxy->ChangeMainParticleStatus_External(PhysicsObjects, bIsMain);
}

Chaos::FPhysicsObjectHandle UClusterUnionComponent::FindChildPhysicsObjectByShapeIndex(int32 Index) const
{
	if (!PerShapeComponentBone.IsValidIndex(Index))
	{
		return nullptr;
	}
	return PerShapeComponentBone[Index].Get();
}