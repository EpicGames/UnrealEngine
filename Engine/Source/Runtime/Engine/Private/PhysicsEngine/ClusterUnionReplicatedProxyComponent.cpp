// Copyright Epic Games, Inc. All Rights Reserved.

#include "PhysicsEngine/ClusterUnionReplicatedProxyComponent.h"

#include "Engine/World.h"
#include "Net/Core/PushModel/PushModel.h"
#include "Net/UnrealNetwork.h"
#include "PhysicsEngine/ClusterUnionComponent.h"
#include "TimerManager.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ClusterUnionReplicatedProxyComponent)

namespace ClusterUnionCVars
{
	bool ClusterUnionUseFlushNetDormancy = true;
	FAutoConsoleVariableRef CVarClusterUnionUseFlushNetDormancy(TEXT("p.Chaos.CU.UseFlushNetDormancy"), ClusterUnionUseFlushNetDormancy, TEXT("When true it will flush the net dormancy of the owner the next frame instead of awaking the actor"));
}

UClusterUnionReplicatedProxyComponent::UClusterUnionReplicatedProxyComponent(const FObjectInitializer& ObjectInitializer)
	: UActorComponent(ObjectInitializer)
{
	SetIsReplicatedByDefault(true);

	ParentClusterUnion = nullptr;
	ChildClusteredComponent = nullptr;

	bNetUpdateParentClusterUnion = false;
	bNetUpdateChildClusteredComponent = false;
	bNetUpdateParticleBoneIds = false;
	bNetUpdateParticleChildToParents = false;
}

void UClusterUnionReplicatedProxyComponent::GetLifetimeReplicatedProps(TArray<FLifetimeProperty>& OutLifetimeProps) const
{
	UActorComponent::GetLifetimeReplicatedProps(OutLifetimeProps);

	FDoRepLifetimeParams Params;
	Params.bIsPushBased = true;

	DOREPLIFETIME_WITH_PARAMS_FAST(UClusterUnionReplicatedProxyComponent, ParentClusterUnion, Params);
	DOREPLIFETIME_WITH_PARAMS_FAST(UClusterUnionReplicatedProxyComponent, ChildClusteredComponent, Params);
	DOREPLIFETIME_WITH_PARAMS_FAST(UClusterUnionReplicatedProxyComponent, ParticleBoneIds, Params);
	DOREPLIFETIME_WITH_PARAMS_FAST(UClusterUnionReplicatedProxyComponent, ParticleChildToParents, Params);
}

void UClusterUnionReplicatedProxyComponent::EndPlay(const EEndPlayReason::Type EndPlayReason)
{
	QUICK_SCOPE_CYCLE_COUNTER(UClusterUnionReplicatedProxyComponent_EndPlay);

	UActorComponent::EndPlay(EndPlayReason);

	MarkPendingDeletion();

	if (AActor* Owner = GetOwner(); Owner && DeferSetChildToParentHandle.IsValid())
	{
		Owner->GetWorldTimerManager().ClearTimer(DeferSetChildToParentHandle);
	}

	if (!GetOwner()->HasAuthority() && ParentClusterUnion.IsValid() && ChildClusteredComponent.IsValid())
	{
		ParentClusterUnion->RemoveComponentFromCluster(ChildClusteredComponent.Get());
	}
}

void UClusterUnionReplicatedProxyComponent::SetParentClusterUnion(UClusterUnionComponent* InComponent)
{
	if (ParentClusterUnion != InComponent)
	{
		FlushNetDormancyIfNeeded();

		ParentClusterUnion = InComponent;
		MARK_PROPERTY_DIRTY_FROM_NAME(UClusterUnionReplicatedProxyComponent, ParentClusterUnion, this);
	}
}

void UClusterUnionReplicatedProxyComponent::SetChildClusteredComponent(UPrimitiveComponent* InComponent)
{
	if (ChildClusteredComponent != InComponent)
	{
		FlushNetDormancyIfNeeded();

		ChildClusteredComponent = InComponent;
		MARK_PROPERTY_DIRTY_FROM_NAME(UClusterUnionReplicatedProxyComponent, ChildClusteredComponent, this);
	}
}

void UClusterUnionReplicatedProxyComponent::SetParticleBoneIds(const TArray<int32>& InIds)
{
	if(ParticleBoneIds != InIds)
	{ 
		FlushNetDormancyIfNeeded();

		ParticleBoneIds = InIds;

		ParticleChildToParents.Reset(InIds.Num());
		for (int32 Index = 0; Index < InIds.Num(); ++Index)
		{
			ParticleChildToParents.Add(FTransform::Identity);
		}
		MARK_PROPERTY_DIRTY_FROM_NAME(UClusterUnionReplicatedProxyComponent, ParticleBoneIds, this);
	}
}

void UClusterUnionReplicatedProxyComponent::SetParticleChildToParent(int32 BoneId, const FTransform& ChildToParent)
{
	int32 Index = INDEX_NONE;
	if (ParticleBoneIds.Find(BoneId, Index))
	{
		if(!ParticleChildToParents[Index].Equals(ChildToParent))
		{ 
			FlushNetDormancyIfNeeded();

			ParticleChildToParents[Index] = ChildToParent;
			MARK_PROPERTY_DIRTY_FROM_NAME(UClusterUnionReplicatedProxyComponent, ParticleChildToParents, this);
		}
	}
}

void UClusterUnionReplicatedProxyComponent::OnRep_ParentClusterUnion()
{
	bNetUpdateParentClusterUnion = true;
}

void UClusterUnionReplicatedProxyComponent::OnRep_ChildClusteredComponent()
{
	bNetUpdateChildClusteredComponent = true;
}

void UClusterUnionReplicatedProxyComponent::OnRep_ParticleBoneIds()
{
	bNetUpdateParticleBoneIds = true;
}

void UClusterUnionReplicatedProxyComponent::OnRep_ParticleChildToParents()
{
	bNetUpdateParticleChildToParents = true;
}

void UClusterUnionReplicatedProxyComponent::PostRepNotifies()
{
	QUICK_SCOPE_CYCLE_COUNTER(UClusterUnionReplicatedProxyComponent_PostRepNotifies);

	UActorComponent::PostRepNotifies();

	if (IsPendingDeletion())
	{
		return;
	}

	const bool bIsValid = ParentClusterUnion.IsValid() && ChildClusteredComponent.IsValid() && !ParticleBoneIds.IsEmpty();
	if (!bIsValid)
	{
		// If any of the above 3 variables isn't valid yet then it probably means they haven't been replicated yet (either by the cluster union proxy component or externally in the case of actors/components).
		// This should be a relatively rare situation - but could cause the client to never attempt to add into the cluster union it if happens.
		// If we're still waiting on some replication - then wait until the next natural call to PostRepNotifies.
		// Otherwise, force a call to PostRepNotifies on the next tick.
		const bool bIsStillWaitingOnReplication = !bNetUpdateParentClusterUnion || !bNetUpdateChildClusteredComponent || !bNetUpdateParticleBoneIds || !bNetUpdateParticleChildToParents;
		if (!bIsStillWaitingOnReplication)
		{
			if (UWorld* World = GetWorld())
			{
				World->GetTimerManager().SetTimerForNextTick(this, &UClusterUnionReplicatedProxyComponent::PostRepNotifies);
			}
		}
		return;
	}

	// These three properties should only get set once when the component is created.
	const bool bIsInitialReplication = bNetUpdateParentClusterUnion || bNetUpdateChildClusteredComponent || bNetUpdateParticleBoneIds;
	if (bIsInitialReplication)
	{
		AddComponentToCluster();

		bNetUpdateParentClusterUnion = false;
		bNetUpdateChildClusteredComponent = false;
		bNetUpdateParticleBoneIds = false;
	}

	if ((bNetUpdateParticleChildToParents || bIsInitialReplication) && ParticleBoneIds.Num() == ParticleChildToParents.Num())
	{
		// This particular bit can't happen until *after* we add the component to the cluster union. There's an additional deferral
		// in AddComponentToCluster that we have to wait for.
		if (!DeferSetChildToParentHandle.IsValid())
		{
			DeferSetChildToParentChildUntilClusteredComponentInParentUnion();
		}
		bNetUpdateParticleChildToParents = false;
	}
}

void UClusterUnionReplicatedProxyComponent::FlushNetDormancyIfNeeded()
{
	if (!ClusterUnionCVars::ClusterUnionUseFlushNetDormancy)
	{
		return;
	}

	if (AActor* Owner = GetOwner())
	{
		Owner->FlushNetDormancy();
	}
}

void UClusterUnionReplicatedProxyComponent::ResetTransientState()
{
	LastSyncedBoneIds.Reset();

	if (AActor* Owner = GetOwner(); Owner && DeferSetChildToParentHandle.IsValid())
	{
		Owner->GetWorldTimerManager().ClearTimer(DeferSetChildToParentHandle);
	}
}

void UClusterUnionReplicatedProxyComponent::AddComponentToCluster()
{
	if (!ParentClusterUnion.IsValid() || !ChildClusteredComponent.IsValid() || IsPendingDeletion())
	{
		return;
	}

	// Need to check if we're *losing* bones instead and handle that situation as well as adding new bones into the cluster union.
	// This extra check also does a bit of prevention on the client side from adding duplicate particles into the cluster union.
	TSet<int32> NewBoneIdSet{ ParticleBoneIds };

	TArray<int32> ToAdd;
	ToAdd.Reserve(NewBoneIdSet.Num());

	for (int32 BoneId : NewBoneIdSet)
	{
		if (!LastSyncedBoneIds.Contains(BoneId))
		{
			ToAdd.Add(BoneId);
		}
	}

	if (!ToAdd.IsEmpty())
	{
		ParentClusterUnion->AddComponentToCluster(ChildClusteredComponent.Get(), ToAdd);
	}

	TArray<int32> ToRemove;
	ToRemove.Reserve(LastSyncedBoneIds.Num());
	for (int32 BoneId : LastSyncedBoneIds)
	{
		if (!NewBoneIdSet.Contains(BoneId))
		{
			ToRemove.Add(BoneId);
		}
	}

	if (!ToRemove.IsEmpty())
	{
		ParentClusterUnion->RemoveComponentBonesFromCluster(ChildClusteredComponent.Get(), ToRemove);
	}

	LastSyncedBoneIds = NewBoneIdSet;
}

void UClusterUnionReplicatedProxyComponent::DeferSetChildToParentChildUntilClusteredComponentInParentUnion()
{
	if (!ParentClusterUnion.IsValid() || !ChildClusteredComponent.IsValid() || IsPendingDeletion())
	{
		return;
	}

	DeferSetChildToParentHandle.Invalidate();

	if (ParentClusterUnion->IsComponentAdded(ChildClusteredComponent.Get()))
	{
		ParentClusterUnion->ForceSetChildToParent(ChildClusteredComponent.Get(), ParticleBoneIds, ParticleChildToParents);
	}
	else if (AActor* Owner = GetOwner())
	{
		DeferSetChildToParentHandle = Owner->GetWorldTimerManager().SetTimerForNextTick(this, &UClusterUnionReplicatedProxyComponent::DeferSetChildToParentChildUntilClusteredComponentInParentUnion);
	}
}
