// Copyright Epic Games, Inc. All Rights Reserved.

#include "CollisionAnalyzerCapture.h"
#include "Engine/HitResult.h"
#include "Engine/OverlapResult.h"

#if ENABLE_COLLISION_ANALYZER

bool bSkipCapture = false;
bool GCollisionAnalyzerIsRecording = false;

void CaptureOverlap(const UWorld* World, const FPhysicsGeometryCollection& PGeom, const FTransform& InGeomTransform, ECAQueryMode::Type QueryMode, ECollisionChannel TraceChannel,
	const FCollisionQueryParams& Params, const FCollisionResponseParams& ResponseParams, const FCollisionObjectQueryParams& ObjectParams, TArray<FOverlapResult>& Results,
	double CPUTime)
{
	if (bSkipCapture || !GCollisionAnalyzerIsRecording || !IsInGameThread())
	{
		return;
	}

	ECAQueryShape::Type QueryShape = ECAQueryShape::Sphere;
	FVector Dims(0, 0, 0);
	FQuat UseRot = InGeomTransform.GetRotation();
	ConvertGeometryCollection(PGeom, UseRot, QueryShape, Dims);

	TArray<FHitResult> HitResults;
	for (const FOverlapResult& OverlapResult : Results)
	{
		FHitResult NewResult = FHitResult(0.f);
		NewResult.bBlockingHit = OverlapResult.bBlockingHit;
		NewResult.HitObjectHandle = OverlapResult.OverlapObjectHandle;
		NewResult.Component = OverlapResult.Component;
		NewResult.Item = OverlapResult.GetItemIndex();
		HitResults.Add(NewResult);
	}

	TArray<FHitResult> TouchAllResults;
	// TODO: Fill in 'all results' for overlaps

	FCollisionAnalyzerModule::Get()->CaptureQuery(InGeomTransform.GetTranslation(), FVector(0, 0, 0), UseRot, ECAQueryType::GeomOverlap, QueryShape, QueryMode, Dims, TraceChannel, Params, ResponseParams, ObjectParams, HitResults, TouchAllResults, CPUTime);
}

void CaptureOverlap(const UWorld* World, const FCollisionShape& PGeom, const FTransform& InGeomTransform, ECAQueryMode::Type QueryMode, ECollisionChannel TraceChannel,
	const FCollisionQueryParams& Params, const FCollisionResponseParams& ResponseParams, const FCollisionObjectQueryParams& ObjectParams, TArray<FOverlapResult>& Results,
	double CPUTime)
{
	if(bSkipCapture || !GCollisionAnalyzerIsRecording || !IsInGameThread())
	{
		return;
	}

	ECAQueryShape::Type QueryShape = ECAQueryShape::Sphere;
	FVector Dims(0, 0, 0);
	FQuat UseRot = InGeomTransform.GetRotation();
	CollisionShapeToAnalyzerType(PGeom, QueryShape, Dims);

	TArray<FHitResult> HitResults;
	for(const FOverlapResult& OverlapResult : Results)
	{
		FHitResult NewResult = FHitResult(0.f);
		NewResult.bBlockingHit = OverlapResult.bBlockingHit;
		NewResult.HitObjectHandle = OverlapResult.OverlapObjectHandle;
		NewResult.Component = OverlapResult.Component;
		NewResult.Item = OverlapResult.GetItemIndex();
		HitResults.Add(NewResult);
	}

	TArray<FHitResult> TouchAllResults;
	// TODO: Fill in 'all results' for overlaps

	FCollisionAnalyzerModule::Get()->CaptureQuery(InGeomTransform.GetTranslation(), FVector(0, 0, 0), UseRot, ECAQueryType::GeomOverlap, QueryShape, QueryMode, Dims, TraceChannel, Params, ResponseParams, ObjectParams, HitResults, TouchAllResults, CPUTime);
}

#endif // ENABLE_COLLISION_ANALYZER