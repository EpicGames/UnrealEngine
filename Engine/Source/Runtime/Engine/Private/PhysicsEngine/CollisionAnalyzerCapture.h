// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Collision.h"

#if ENABLE_COLLISION_ANALYZER

#include "ICollisionAnalyzer.h"
#include "CollisionAnalyzerModule.h"
#include "Engine/HitResult.h"
#include "Physics/PhysicsInterfaceCore.h"
#include "Physics/PhysicsInterfaceUtils.h"

#include "Chaos/Capsule.h"
#include "Chaos/Sphere.h"

extern bool bSkipCapture;
extern bool GCollisionAnalyzerIsRecording;

/** Util to convert from PhysX shape and rotation to unreal shape enum, dimension vector and rotation */
inline static void ConvertGeometryCollection(const FPhysicsGeometryCollection& GeomCollection, FQuat& InOutRot, ECAQueryShape::Type& OutQueryShape, FVector& OutDims)
{
	OutQueryShape = ECAQueryShape::Capsule;
	OutDims = FVector(0, 0, 0);
	
	switch (GeomCollection.GetType())
	{
		case ECollisionShapeType::Capsule:
		{
			OutQueryShape = ECAQueryShape::Capsule;
			const auto& CapsuleGeom = GeomCollection.GetCapsuleGeometry();
			OutDims = FVector(CapsuleGeom.GetRadiusf(), CapsuleGeom.GetRadiusf(), CapsuleGeom.GetHeightf() * 0.5f + CapsuleGeom.GetRadiusf());
			break;
		}

		case ECollisionShapeType::Sphere:
		{
			OutQueryShape = ECAQueryShape::Sphere;
			const auto& SphereGeom = GeomCollection.GetSphereGeometry();
			OutDims = FVector(SphereGeom.GetRadiusf());
			break;
		}

		case ECollisionShapeType::Box:
		{
			OutQueryShape = ECAQueryShape::Box;
			const auto& BoxGeom = GeomCollection.GetBoxGeometry();
			OutDims = FVector(BoxGeom.Extents() * 0.5f);
			break;
		}

		case ECollisionShapeType::Convex:
		{
			OutQueryShape = ECAQueryShape::Convex;
			break;
		}

		default:
			UE_LOGF(LogCollision, Warning, "CaptureGeomSweep: Unknown geom type.");
	}
}

inline bool CollisionShapeToAnalyzerType(const FCollisionShape& InShape, ECAQueryShape::Type& OutType, FVector& OutDims)
{
	switch(InShape.ShapeType)
	{
		case ECollisionShape::Sphere:
		{
			OutType = ECAQueryShape::Sphere;
			OutDims = FVector(InShape.GetSphereRadius());
			return true;
		}
		case ECollisionShape::Capsule:
		{
			OutType = ECAQueryShape::Capsule;
			const float CapsuleRadius = InShape.GetCapsuleRadius();
			OutDims = FVector(CapsuleRadius, CapsuleRadius, InShape.GetCapsuleHalfHeight() + CapsuleRadius);
			return true;
		}
		case ECollisionShape::Box:
		{
			OutType = ECAQueryShape::Box;
			OutDims = InShape.GetBox();
			return true;
		}
		default:
			break;
	}

	UE_LOGF(LogCollision, Warning, "CaptureGeomSweep: Unknown geom type.");
	return false;
}

/** Util to extract type and dimensions from physx geom being swept, and pass info to CollisionAnalyzer, if its recording */
inline void CaptureGeomSweep(const UWorld* World, const FVector& Start, const FVector& End, const FQuat& Rot, ECAQueryMode::Type QueryMode, const FCollisionShape& PGeom, ECollisionChannel TraceChannel, const struct FCollisionQueryParams& Params, const struct FCollisionResponseParams& ResponseParams, const struct FCollisionObjectQueryParams& ObjectParams, const TArray<FHitResult>& Results, double CPUTime)
{
	if (bSkipCapture || !GCollisionAnalyzerIsRecording || !IsInGameThread())
	{
		return;
	}

	// Convert from PhysX to Unreal types
	ECAQueryShape::Type QueryShape;
	FVector Dims(0, 0, 0);
	CollisionShapeToAnalyzerType(PGeom, QueryShape, Dims);

	// Do a touch all query to find things we _didn't_ hit
	bSkipCapture = true;
	TArray<FHitResult> TouchAllResults;
	FPhysicsInterface::GeomSweepMulti(World, PGeom, Rot, TouchAllResults, Start, End, DefaultCollisionChannel, Params, ResponseParams, FCollisionObjectQueryParams(FCollisionObjectQueryParams::InitType::AllObjects));

	bSkipCapture = false;

	// Now tell analyzer
	FCollisionAnalyzerModule::Get()->CaptureQuery(Start, End, Rot, ECAQueryType::GeomSweep, QueryShape, QueryMode, Dims, TraceChannel, Params, ResponseParams, ObjectParams, Results, TouchAllResults, CPUTime);
}

/** Util to extract type and dimensions from physx geom being swept, and pass info to CollisionAnalyzer, if its recording */
inline void CaptureGeomSweep(const UWorld* World, const FVector& Start, const FVector& End, const FQuat& GeomRot, ECAQueryMode::Type QueryMode, const FPhysicsGeometryCollection& GeomCollection, ECollisionChannel TraceChannel, const struct FCollisionQueryParams& Params, const struct FCollisionResponseParams& ResponseParams, const struct FCollisionObjectQueryParams& ObjectParams, const TArray<FHitResult>& Results, double CPUTime)
{
	if(bSkipCapture || !GCollisionAnalyzerIsRecording || !IsInGameThread())
	{
		return;
	}

	// Convert from PhysX to Unreal types
	ECAQueryShape::Type QueryShape = ECAQueryShape::Sphere;
	FVector Dims(0, 0, 0);
	FQuat UseRot = GeomRot;
	ConvertGeometryCollection(GeomCollection, UseRot, QueryShape, Dims);

	// Do a touch all query to find things we _didn't_ hit
	bSkipCapture = true;
	TArray<FHitResult> TouchAllResults;
	FPhysicsInterface::GeomSweepMulti(World, GeomCollection, UseRot, TouchAllResults, Start, End, DefaultCollisionChannel, Params, ResponseParams, FCollisionObjectQueryParams(FCollisionObjectQueryParams::InitType::AllObjects));

	bSkipCapture = false;

	// Now tell analyzer
	FCollisionAnalyzerModule::Get()->CaptureQuery(Start, End, UseRot, ECAQueryType::GeomSweep, QueryShape, QueryMode, Dims, TraceChannel, Params, ResponseParams, ObjectParams, Results, TouchAllResults, CPUTime);
}

/** Util to capture a raycast with the CollisionAnalyzer if recording */
inline void CaptureRaycast(const UWorld* World, const FVector& Start, const FVector& End, ECAQueryMode::Type QueryMode, ECollisionChannel TraceChannel, const struct FCollisionQueryParams& Params, const struct FCollisionResponseParams& ResponseParams, const struct FCollisionObjectQueryParams& ObjectParams, const TArray<FHitResult>& Results, double CPUTime)
{
	if (bSkipCapture || !GCollisionAnalyzerIsRecording || !IsInGameThread())
	{
		return;
	}

	// Do a touch all query to find things we _didn't_ hit
	bSkipCapture = true;
	TArray<FHitResult> TouchAllResults;
	FPhysicsInterface::RaycastMulti(World, TouchAllResults, Start, End, DefaultCollisionChannel, Params, ResponseParams, FCollisionObjectQueryParams(FCollisionObjectQueryParams::InitType::AllObjects));
	bSkipCapture = false;

	FCollisionAnalyzerModule::Get()->CaptureQuery(Start, End, FQuat::Identity, ECAQueryType::Raycast, ECAQueryShape::Sphere, QueryMode, FVector(0, 0, 0), TraceChannel, Params, ResponseParams, ObjectParams, Results, TouchAllResults, CPUTime);
}

void CaptureOverlap(const UWorld* World, const FPhysicsGeometryCollection& PGeom, const FTransform& InGeomTransform, ECAQueryMode::Type QueryMode, ECollisionChannel TraceChannel, const struct FCollisionQueryParams& Params, const struct FCollisionResponseParams& ResponseParams, const struct FCollisionObjectQueryParams& ObjectParams, TArray<FOverlapResult>& Results, double CPUTime);

void CaptureOverlap(const UWorld* World, const FCollisionShape& PGeom, const FTransform& InGeomTransform, ECAQueryMode::Type QueryMode, ECollisionChannel TraceChannel, const struct FCollisionQueryParams& Params, const struct FCollisionResponseParams& ResponseParams, const struct FCollisionObjectQueryParams& ObjectParams, TArray<FOverlapResult>& Results, double CPUTime);

#define STARTQUERYTIMER() double StartTime = FPlatformTime::Seconds()
#define CAPTUREGEOMSWEEP(World, Start, End, Rot, QueryMode, PGeom, TraceChannel, Params, ResponseParam, ObjectParam, Results) if (GCollisionAnalyzerIsRecording && IsInGameThread()) { CaptureGeomSweep(World, Start, End, Rot, QueryMode, PGeom, TraceChannel, Params, ResponseParam, ObjectParam, Results, FPlatformTime::Seconds() - StartTime); }
#define CAPTURERAYCAST(World, Start, End, QueryMode, TraceChannel, Params, ResponseParam, ObjectParam, Results)	if (GCollisionAnalyzerIsRecording && IsInGameThread()) { CaptureRaycast(World, Start, End, QueryMode, TraceChannel, Params, ResponseParam, ObjectParam, Results, FPlatformTime::Seconds() - StartTime); }
#define CAPTUREGEOMOVERLAP(World, PGeom, PGeomPose, QueryMode, TraceChannel, Params, ResponseParams, ObjectParams, Results)	if (GCollisionAnalyzerIsRecording && IsInGameThread()) { CaptureOverlap(World, PGeom, PGeomPose, QueryMode, TraceChannel, Params, ResponseParams, ObjectParams, Results, FPlatformTime::Seconds() - StartTime); }


#else

#define STARTQUERYTIMER() 
#define CAPTUREGEOMSWEEP(...)
#define CAPTURERAYCAST(...)
#define CAPTUREGEOMOVERLAP(...)

#endif // ENABLE_COLLISION_ANALYZER