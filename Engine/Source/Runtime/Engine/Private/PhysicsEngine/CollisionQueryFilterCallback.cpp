// Copyright Epic Games, Inc. All Rights Reserved.

#include "PhysicsEngine/CollisionQueryFilterCallback.h"
#include "PhysicsEngine/ScopedSQHitchRepeater.h"
#include "Physics/PhysicsFiltering.h"
#include "Components/PrimitiveComponent.h"

#include "Physics/Experimental/ChaosInterfaceWrapper.h"
#include "Chaos/GeometryParticles.h"

ECollisionQueryHitType ConvertToHitType(const Chaos::Filter::ENarrowFilterResult FilterResult)
{
	switch (FilterResult)
	{
		case Chaos::Filter::ENarrowFilterResult::None: return ECollisionQueryHitType::None;
		case Chaos::Filter::ENarrowFilterResult::Overlap: return ECollisionQueryHitType::Touch;
		case Chaos::Filter::ENarrowFilterResult::Block: return ECollisionQueryHitType::Block;
		default: return ECollisionQueryHitType::None;
	}
}

ECollisionQueryHitType FCollisionQueryFilterCallback::CalcQueryHitType(const FQueryFilterData& QueryFilter, const FShapeFilterData& ShapeFilter, bool bPreFilter)
{
	const ECollisionQueryHitType HitType = ConvertToHitType(QueryFilter.NarrowFilter(ShapeFilter));
	if (HitType == ECollisionQueryHitType::None)
	{
		return HitType;
	}

	// There's some edge cases to handle...
	const FQueryFilterData::EQueryType QueryType = QueryFilter.GetQueryType();
	if (QueryType == FQueryFilterData::EQueryType::ObjectType)
	{
		// The regular object type filter logic just chooses None/Block.
		// For pre-filtering we want all results, so convert block to touch in the multi case.
		if (bPreFilter && QueryFilter.IsMultiQuery())
		{
			return ECollisionQueryHitType::Touch;
		}
	}
	else
	{
		// If query channel is Touch All, then just return touch.
		// Will have to figure out this special case when channel index is converted to registered channel masks...
		if (QueryFilter.GetCollisionChannelIndex() == ECC_OverlapAll_Deprecated)
		{
			return ECollisionQueryHitType::Touch;
		}
	}
	return HitType;
}

ECollisionQueryHitType FCollisionQueryFilterCallback::PostFilter(const FQueryFilterData& FilterData, const ChaosInterface::FQueryHit& Hit)
{
	return PostFilterImp(FilterData, Hit);
}

ECollisionQueryHitType FCollisionQueryFilterCallback::PreFilter(const FQueryFilterData& FilterData, const Chaos::FPerShapeData& Shape, const Chaos::FGeometryParticle& Actor)
{
	return PreFilterImp(FilterData, Shape, Actor);
}

ECollisionQueryHitType FCollisionQueryFilterCallback::PostFilter(const FQueryFilterData& FilterData, const ChaosInterface::FPTQueryHit& Hit)
{
	return PostFilterImp(FilterData, Hit);
}

ECollisionQueryHitType FCollisionQueryFilterCallback::PreFilter(const FQueryFilterData& FilterData, const Chaos::FPerShapeData& Shape, const Chaos::FGeometryParticleHandle& Actor)
{
	return PreFilterImp(FilterData, Shape, Actor);
}

template <typename TParticle>
ECollisionQueryHitType FCollisionQueryFilterCallback::PreFilterBaseImp(const FQueryFilterData& FilterData, const Chaos::FPerShapeData& Shape, const TParticle& Actor)
{
	//SCOPE_CYCLE_COUNTER(STAT_Collision_PreFilter);

	if (!Shape.GetQueryEnabled())
	{
		return ECollisionQueryHitType::None;
	}

	const FCombinedShapeFilterData CombinedShapeFilter = ChaosInterface::GetCombinedShapeFilterData(Shape);

	FBodyInstance* BodyInstance = nullptr;

	if constexpr (std::is_same<TParticle, Chaos::FGeometryParticle>::value)
	{
#if ENABLE_PREFILTER_LOGGING || DETECT_SQ_HITCHES
		BodyInstance = ChaosInterface::GetUserData(Actor);
#endif // ENABLE_PREFILTER_LOGGING || DETECT_SQ_HITCHES
	}

	return PreFilterImp(FilterData, CombinedShapeFilter, BodyInstance);
}

ECollisionQueryHitType FCollisionQueryFilterCallback::PreFilterImp(const FQueryFilterData& FilterData, const Chaos::FPerShapeData& Shape, const Chaos::FGeometryParticle& Actor)
{
	return PreFilterBaseImp(FilterData, Shape, Actor);
}

ECollisionQueryHitType FCollisionQueryFilterCallback::PreFilterImp(const FQueryFilterData& FilterData, const Chaos::FPerShapeData& Shape, const Chaos::FGeometryParticleHandle& Actor)
{
	return PreFilterBaseImp(FilterData, Shape, Actor);
}

ECollisionQueryHitType FCollisionQueryFilterCallback::PreFilterImp(const FQueryFilterData& FilterData, const FCombinedShapeFilterData& CombinedShapeFilter, const FBodyInstance* BodyInstance)
{
#if DETECT_SQ_HITCHES
	FPreFilterRecord* PreFilterRecord = nullptr;
	if (bRecordHitches && (IsInGameThread() || FSQHitchRepeaterCVars::SQHitchDetectionForceNames))
	{
		PreFilterHitchInfo.AddZeroed();
		PreFilterRecord = &PreFilterHitchInfo[PreFilterHitchInfo.Num() - 1];
		
		if(BodyInstance)
		{
			if(UPrimitiveComponent* OwnerComp = BodyInstance->OwnerComponent.Get())
			{
				PreFilterRecord->OwnerComponentReadableName = OwnerComp->GetReadableName();
			}
		}
	}
#endif

	const FInstanceData& ShapeInstanceData = CombinedShapeFilter.GetInstanceData();
	const FShapeFilterData& ShapeFilter = CombinedShapeFilter.GetShapeFilterData();

#if ENABLE_PREFILTER_LOGGING
	static bool bLoggingEnabled = false;
	if (bLoggingEnabled)
	{
		if(BodyInstance && BodyInstance->OwnerComponent.IsValid())
		{
			UE_LOG(LogCollision, Warning, TEXT("[PREFILTER] against %s[%s] : About to check "),
				(BodyInstance->OwnerComponent.Get()->GetOwner()) ? *BodyInstance->OwnerComponent.Get()->GetOwner()->GetName() : TEXT("NO OWNER"),
				*BodyInstance->OwnerComponent.Get()->GetName());
		}

		UE_LOGF(LogCollision, Warning, "ShapeInstanceData : %ls", *ShapeInstanceData.ToString());
		UE_LOGF(LogCollision, Warning, "ShapeFilter : %ls", *ShapeFilter.ToString());
		UE_LOGF(LogCollision, Warning, "QueryFilter : %ls", *FilterData.ToString());
	}
#endif // ENABLE_PREFILTER_LOGGING

	// Shape : shape's Filter Data
	// Querier : filterData that owns the trace
	const Chaos::EFilterFlags ShapeFlags = ShapeFilter.GetFlags();
	const Chaos::EFilterFlags QuerierFlags = FilterData.GetFlags();
	const Chaos::EFilterFlags CommonFlags = ShapeFlags & QuerierFlags;

	// First check complexity, none of them matches
	if (!(CommonFlags & Chaos::EFilterFlags::SimpleCollision) && !(CommonFlags & Chaos::EFilterFlags::ComplexCollision))
	{
		return (PreFilterReturnValue = ECollisionQueryHitType::None);
	}
	
	ECollisionQueryHitType Result = FCollisionQueryFilterCallback::CalcQueryHitType(FilterData, ShapeFilter, true);

	if (Result == ECollisionQueryHitType::Touch && bIgnoreTouches)
	{
		Result = ECollisionQueryHitType::None;
	}

	if (Result == ECollisionQueryHitType::Block && bIgnoreBlocks)
	{
		Result = ECollisionQueryHitType::None;
	}

	// If not already rejected, check ignore actor and component list.
	if (Result != ECollisionQueryHitType::None)
	{
		// See if we are ignoring the actor this shape belongs to (word0 of shape filterdata is actorID)
		if (IgnoreActors.Contains(ShapeInstanceData.GetOwnerId()))
		{
			//UE_LOGF(LogTemp, Log, "Ignoring Actor: %d", ShapeFilter.word0);
			Result = ECollisionQueryHitType::None;
		}

		if (IgnoreComponents.Contains(ShapeInstanceData.GetComponentId()))
		{
			//UE_LOGF(LogTemp, Log, "Ignoring Component: %d", shape->getSimulationFilterData().word2);
			Result = ECollisionQueryHitType::None;
		}
	}

#if ENABLE_PREFILTER_LOGGING
	if (bLoggingEnabled)
	{
		uint32 QuerierChannel = FilterData.GetCollisionChannelIndex();
		UE_LOGF(LogCollision, Log, "[PREFILTER] Result for Querier [CHANNEL: %d, FLAG: %x] [%d]", QuerierChannel, QuerierFlags, (int32)Result);
	}
#endif // ENABLE_PREFILTER_LOGGING

	if (bIsOverlapQuery && Result == ECollisionQueryHitType::Block)
	{
		Result = ECollisionQueryHitType::Touch;	//In the case of overlaps, physx only understands touches. We do this at the end to ensure all filtering logic based on block vs overlap is correct
	}

#if DETECT_SQ_HITCHES
	if (PreFilterRecord)
	{
		PreFilterRecord->Result = Result;
	}
#endif

	return  (PreFilterReturnValue = Result);
}


ECollisionQueryHitType FCollisionQueryFilterCallback::PostFilterImp(const FQueryFilterData& FilterData, bool bIsOverlap)
{
	if (!bIsSweep)
	{
		return ECollisionQueryHitType::Block;
	}
	else if (bIsOverlap && bDiscardInitialOverlaps)
	{
		return ECollisionQueryHitType::None;
	}
	else
	{
		if (bIsOverlap && PreFilterReturnValue == ECollisionQueryHitType::Block)
		{
			// We want to keep initial blocking overlaps and continue the sweep until a non-overlapping blocking hit.
			// We will later report this hit as a blocking hit when we compute the hit type (using CalcQueryHitType).
			return ECollisionQueryHitType::Touch;
		}

		return PreFilterReturnValue;
	}
}

ECollisionQueryHitType FCollisionQueryFilterCallback::PostFilterImp(const FQueryFilterData& FilterData, const ChaosInterface::FQueryHit& Hit)
{
	// Unused in non-sweeps
	if (!bIsSweep)
	{
		return ECollisionQueryHitType::None;
	}

	const auto& SweepHit = static_cast<const ChaosInterface::FLocationHit&>(Hit);
	const bool bIsOverlap = ChaosInterface::HadInitialOverlap(SweepHit);

	return PostFilterImp(FilterData, bIsOverlap);
}

ECollisionQueryHitType FCollisionQueryFilterCallback::PostFilterImp(const FQueryFilterData& FilterData, const ChaosInterface::FPTQueryHit& Hit)
{
	// Unused in non-sweeps
	if (!bIsSweep)
	{
		return ECollisionQueryHitType::None;
	}

	const auto& SweepHit = static_cast<const ChaosInterface::FPTLocationHit&>(Hit);
	const bool bIsOverlap = ChaosInterface::HadInitialOverlap(SweepHit);

	return PostFilterImp(FilterData, bIsOverlap);
}
