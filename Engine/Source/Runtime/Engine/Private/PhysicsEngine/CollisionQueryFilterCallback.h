// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CollisionQueryFilterCallbackCore.h"
#include "CollisionQueryParams.h"
#include "Physics/PhysicsInterfaceTypes.h"

#define ENABLE_PREFILTER_LOGGING 0

struct FBodyInstance;

/** TArray typedef of components to ignore. */
typedef FCollisionQueryParams::IgnoreComponentsArrayType FilterIgnoreComponentsArrayType;

/** TArray typedef of actors to ignore. */
typedef FCollisionQueryParams::IgnoreActorsArrayType FilterIgnoreActorsArrayType;

class FCollisionQueryFilterCallback : public ICollisionQueryFilterCallback
{
public:
	using FInstanceData = Chaos::Filter::FInstanceData;
	using FQueryFilterData = Chaos::Filter::FQueryFilterData;
	using FShapeFilterData = Chaos::Filter::FShapeFilterData;
	using FCombinedShapeFilterData = Chaos::Filter::FCombinedShapeFilterData;

	/** Result of PreFilter callback. */
	ECollisionQueryHitType PreFilterReturnValue;

	/** List of ComponentIds for this query to ignore */
	const FilterIgnoreComponentsArrayType& IgnoreComponents;

	//~ TODO: It would be nice to rename this to IgnoreSourceObjects, because these might not be actors in
	//~ non-actor workflows (requires deprecation).
	/** List of ActorIds for this query to ignore */
	const FilterIgnoreActorsArrayType& IgnoreActors;

	/** Whether we are doing an overlap query. This is needed to ensure physx results are never blocking (even if they are in terms of unreal)*/
	bool bIsOverlapQuery;

	/** Whether to ignore touches (convert an eTOUCH result to eNONE). */
	bool bIgnoreTouches;

	/** Whether to ignore blocks (convert an eBLOCK result to eNONE). */
	bool bIgnoreBlocks;

	FCollisionQueryFilterCallback(const FCollisionQueryParams& InQueryParams, bool bInIsSweep)
		: IgnoreComponents(InQueryParams.GetIgnoredComponents())
		, IgnoreActors(InQueryParams.GetIgnoredSourceObjects())
#if DETECT_SQ_HITCHES
		, bRecordHitches(false)
#endif
		, bIsSweep(bInIsSweep)
	{
		PreFilterReturnValue = ECollisionQueryHitType::None;
		bIsOverlapQuery = false;
		bIgnoreTouches = InQueryParams.bIgnoreTouches;
		bIgnoreBlocks = InQueryParams.bIgnoreBlocks;
		bDiscardInitialOverlaps = !InQueryParams.bFindInitialOverlaps;
	}
	~FCollisionQueryFilterCallback() = default;

	static ECollisionQueryHitType CalcQueryHitType(const FQueryFilterData& QueryFilter, const FShapeFilterData& ShapeFilter, bool bPreFilter = false);

	virtual ECollisionQueryHitType PostFilter(const FQueryFilterData& FilterData, const ChaosInterface::FQueryHit& Hit) override;
	virtual ECollisionQueryHitType PostFilter(const FQueryFilterData& FilterData, const ChaosInterface::FPTQueryHit& Hit) override;

	virtual ECollisionQueryHitType PreFilter(const FQueryFilterData& FilterData, const Chaos::FPerShapeData& Shape, const Chaos::FGeometryParticle& Actor) override;
	virtual ECollisionQueryHitType PreFilter(const FQueryFilterData& FilterData, const Chaos::FPerShapeData& Shape, const Chaos::FGeometryParticleHandle& Actor) override;

#if DETECT_SQ_HITCHES
	// Util struct to record what preFilter was called with
	struct FPreFilterRecord
	{
		FString OwnerComponentReadableName;
		ECollisionQueryHitType Result;
	};

	TArray<FPreFilterRecord> PreFilterHitchInfo;
	bool bRecordHitches;
#endif
	bool bDiscardInitialOverlaps;
	bool bIsSweep;

private:
	template <typename TParticle>
	ECollisionQueryHitType PreFilterBaseImp(const FQueryFilterData& FilterData, const Chaos::FPerShapeData& Shape, const TParticle& Actor);

	ECollisionQueryHitType PreFilterImp(const FQueryFilterData& FilterData, const Chaos::FPerShapeData& Shape, const Chaos::FGeometryParticle& Actor);
	ECollisionQueryHitType PreFilterImp(const FQueryFilterData& FilterData, const Chaos::FPerShapeData& Shape, const Chaos::FGeometryParticleHandle& Actor);
	ECollisionQueryHitType PreFilterImp(const FQueryFilterData& FilterData, const FCombinedShapeFilterData& CombinedShapeFilterData, const FBodyInstance* BodyInstance);

	ECollisionQueryHitType PostFilterImp(const FQueryFilterData& FilterData, bool bIsOverlap);
	ECollisionQueryHitType PostFilterImp(const FQueryFilterData& FilterData, const ChaosInterface::FQueryHit& Hit);
	ECollisionQueryHitType PostFilterImp(const FQueryFilterData& FilterData, const ChaosInterface::FPTQueryHit& Hit);
};

