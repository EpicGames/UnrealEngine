// Copyright Epic Games, Inc. All Rights Reserved.

#include "PhysicsEngine/CollisionStructure.h"


//////////////////////////////////////////////////////////////////////////

FCollisionStructure::FCollisionStructure()
{

}

int32 FCollisionStructure::CreateCollisionEntry(const FKAggregateGeom& InGeom, const FTransform& InTransform, const FCollisionFilterData& InQueryFilter, const FCollisionFilterData& InSimFilter)
{
	int32 NewEntryIndex = INDEX_NONE;

	if (FreeList.Num() > 0)
	{
		NewEntryIndex = FreeList.Pop();
	}
	else
	{
		NewEntryIndex = ValidFlag.AddUninitialized();
		Bounds.AddUninitialized();
		Geom.AddUninitialized();
		Transform.AddUninitialized();
		QueryFilter.AddUninitialized();
		SimFilter.AddUninitialized();
	}

	ValidFlag[NewEntryIndex] = true;
	Geom[NewEntryIndex] = InGeom;
	Transform[NewEntryIndex] = InTransform;
	QueryFilter[NewEntryIndex] = InQueryFilter;
	SimFilter[NewEntryIndex] = InSimFilter;
	UpdateBounds(NewEntryIndex);

	return NewEntryIndex;
}

void FCollisionStructure::DestroyCollisionEntry(int32 EntryIndex)
{
	if (EntryIsValid(EntryIndex))
	{
		ValidFlag[EntryIndex] = false; // set flag that this entry is now invalid
		FreeList.Add(EntryIndex); // add index to free list
	}
}

void FCollisionStructure::SetEntryTransform(int32 EntryIndex, const FTransform& InTransform)
{
	if (EntryIsValid(EntryIndex))
	{
		Transform[EntryIndex] = InTransform;
		UpdateBounds(EntryIndex);
	}
}

void FCollisionStructure::UpdateBounds(int32 EntryIndex)
{
	check(EntryIsValid(EntryIndex));
	Bounds[EntryIndex] = Geom[EntryIndex].CalcAABB(Transform[EntryIndex]);
}

bool FCollisionStructure::RaycastSingle(const FVector& Start, const FVector& End, FHitResult& OutHit, const FCollisionFilterData& InQueryFilter)
{
	bool bBlockingHit = false;

	const FVector Delta = End - Start;
	const float DeltaMag = Delta.Size();
	if (DeltaMag > UE_KINDA_SMALL_NUMBER)
	{
		const FVector OneOverDelta = Delta/ DeltaMag;

		const int32 NumEntries = ValidFlag.Num();
		for (int32 EntryIdx = 0; EntryIdx < NumEntries; EntryIdx++)
		{
			// First check if valid entry
			if (ValidFlag[EntryIdx])
			{
				// Now check bounds
				if (FMath::LineBoxIntersection(Bounds[EntryIdx], Start, End, Delta, OneOverDelta))
				{
					// Now check filtering

				}
			}
		}
	}

	return bBlockingHit;
}


bool FCollisionStructure::EntryIsValid(int32 EntryIndex)
{
	return ValidFlag.IsValidIndex(EntryIndex) && ValidFlag[EntryIndex];
}
