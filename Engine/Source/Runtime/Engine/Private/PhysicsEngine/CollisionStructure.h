// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PhysicsEngine/AggregateGeom.h"
#include "PhysicsEngine/CollisionQueryFilterCallback.h"

struct FHitResult;

class UE_DEPRECATED(5.8, "This was unused and will be removed") FCollisionStructure
{
	TArray<bool>					ValidFlag;
	TArray<FBox>					Bounds;
	TArray<FKAggregateGeom>			Geom;
	TArray<FTransform>				Transform;
	TArray<FCollisionFilterData>	QueryFilter;
	TArray<FCollisionFilterData>	SimFilter;

	TArray<int32>					FreeList;

	FCollisionStructure();
	~FCollisionStructure() {}

public:
	int32 CreateCollisionEntry(const FKAggregateGeom& InGeom, const FTransform& InTransform, const FCollisionFilterData& InQueryFilter, const FCollisionFilterData& InSimFilter);

	void DestroyCollisionEntry(int32 EntryIndex);

	void SetEntryTransform(int32 EntryIndex, const FTransform& InTransform);

	bool RaycastSingle(const FVector& Start, const FVector& End, FHitResult& OutHit, const FCollisionFilterData& QueryFilter);

	bool EntryIsValid(int32 EntryIndex);

private:

	void UpdateBounds(int32 EntryIndex);
};