// Copyright Epic Games, Inc. All Rights Reserved.

#include "PhysicsEngine/ConstraintDrives.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ConstraintDrives)

const bool bIsAccelerationDrive = true;

FConstraintDrive::FConstraintDrive()
	: Stiffness(50.f)
	, Damping(1.f)
	, MaxForce(0.f)
	, bEnablePositionDrive(false)
	, bEnableVelocityDrive(false)
{
}

FLinearDriveConstraint::FLinearDriveConstraint()
	: PositionTarget(ForceInit)
	, VelocityTarget(ForceInit)
{
}

FAngularDriveConstraint::FAngularDriveConstraint()
	: OrientationTarget(ForceInit)
	, AngularVelocityTarget(ForceInit)
	, AngularDriveMode(EAngularDriveMode::SLERP)
{
}

void FLinearDriveConstraint::SetLinearPositionDrive(bool bEnableXDrive, bool bEnableYDrive, bool bEnableZDrive)
{
	XDrive.bEnablePositionDrive = bEnableXDrive;
	YDrive.bEnablePositionDrive = bEnableYDrive;
	ZDrive.bEnablePositionDrive = bEnableZDrive;
}

void FLinearDriveConstraint::SetLinearVelocityDrive(bool bEnableXDrive, bool bEnableYDrive, bool bEnableZDrive)
{
	XDrive.bEnableVelocityDrive = bEnableXDrive;
	YDrive.bEnableVelocityDrive = bEnableYDrive;
	ZDrive.bEnableVelocityDrive = bEnableZDrive;
}

void FAngularDriveConstraint::SetOrientationDriveTwistAndSwing(bool InEnableTwistDrive, bool InEnableSwingDrive)
{
	SwingDrive.bEnablePositionDrive = InEnableSwingDrive;
	TwistDrive.bEnablePositionDrive = InEnableTwistDrive;
}

void FAngularDriveConstraint::SetOrientationDriveSLERP(bool InEnableSLERP)
{
	SlerpDrive.bEnablePositionDrive = InEnableSLERP;
}

void FAngularDriveConstraint::SetAngularVelocityDriveTwistAndSwing(bool InEnableTwistDrive, bool InEnableSwingDrive)
{
	SwingDrive.bEnableVelocityDrive = InEnableSwingDrive;
	TwistDrive.bEnableVelocityDrive = InEnableTwistDrive;
}

void FAngularDriveConstraint::SetAngularVelocityDriveSLERP(bool InEnableSLERP)
{
	SlerpDrive.bEnableVelocityDrive = InEnableSLERP;
}

void FConstraintDrive::SetDriveParams(float InStiffness, float InDamping, float InForceLimit)
{
	Stiffness = InStiffness;
	Damping = InDamping;
	MaxForce = InForceLimit;
}

void FLinearDriveConstraint::SetDriveParams(float InStiffness, float InDamping, float InForceLimit)
{
	XDrive.SetDriveParams(InStiffness, InDamping, InForceLimit);
	YDrive.SetDriveParams(InStiffness, InDamping, InForceLimit);
	ZDrive.SetDriveParams(InStiffness, InDamping, InForceLimit);
}

void FLinearDriveConstraint::SetDriveParams(const FVector& InStiffness, const FVector& InDamping, const FVector& InForceLimit)
{
	XDrive.SetDriveParams(InStiffness.X, InDamping.X, InForceLimit.X);
	YDrive.SetDriveParams(InStiffness.Y, InDamping.Y, InForceLimit.Y);
	ZDrive.SetDriveParams(InStiffness.Z, InDamping.Z, InForceLimit.Z);
}

void FLinearDriveConstraint::GetDriveParams(float& OutStiffness, float& OutDamping, float& OutForceLimit) const
{
	// we set the same value on all drives, just return XDrive ones
	OutStiffness = XDrive.Stiffness;
	OutDamping = XDrive.Damping;
	OutForceLimit = XDrive.MaxForce;
}

void FLinearDriveConstraint::GetDriveParams(FVector& OutStiffness, FVector& OutDamping, FVector& OutForceLimit) const
{
	OutStiffness.Set(XDrive.Stiffness, YDrive.Stiffness, ZDrive.Stiffness);
	OutDamping.Set(XDrive.Damping, YDrive.Damping, ZDrive.Damping);
	OutForceLimit.Set(XDrive.MaxForce, YDrive.MaxForce, ZDrive.MaxForce);
}

void FLinearDriveConstraint::SetAccelerationMode(const bool bInAccelerationMode)
{
	bAccelerationMode = bInAccelerationMode;
}

bool FLinearDriveConstraint::GetAccelerationMode() const
{
	return bAccelerationMode;
}

void FAngularDriveConstraint::SetAngularDriveMode(EAngularDriveMode::Type DriveMode)
{
	AngularDriveMode = DriveMode;
}

void FAngularDriveConstraint::SetDriveParams(float InStiffness, float InDamping, float InForceLimit)
{
	SwingDrive.SetDriveParams(InStiffness, InDamping, InForceLimit);
	TwistDrive.SetDriveParams(InStiffness, InDamping, InForceLimit);
	SlerpDrive.SetDriveParams(InStiffness, InDamping, InForceLimit);
}

void FAngularDriveConstraint::SetDriveParams(const FVector& InStiffness, const FVector& InDamping, const FVector& InForceLimit)
{
	SwingDrive.SetDriveParams(InStiffness.X, InDamping.X, InForceLimit.X);
	TwistDrive.SetDriveParams(InStiffness.Y, InDamping.Y, InForceLimit.Y);
	SlerpDrive.SetDriveParams(InStiffness.Z, InDamping.Z, InForceLimit.Z);
}

void FAngularDriveConstraint::GetDriveParams(float& OutStiffness, float& OutDamping, float& OutForceLimit) const
{
	// we set the same value on all drives, just return SwingDrive ones
	OutStiffness = SwingDrive.Stiffness;
	OutDamping = SwingDrive.Damping;
	OutForceLimit = SwingDrive.MaxForce;
}

void FAngularDriveConstraint::GetDriveParams(FVector& OutStiffness, FVector& OutDamping, FVector& OutForceLimit) const
{
	OutStiffness.Set(SwingDrive.Stiffness, TwistDrive.Stiffness, TwistDrive.MaxForce);
	OutDamping.Set(SwingDrive.Damping, TwistDrive.Damping, TwistDrive.MaxForce);
	OutForceLimit.Set(SwingDrive.MaxForce, TwistDrive.Damping, TwistDrive.MaxForce);

}

void FAngularDriveConstraint::SetAccelerationMode(const bool bInAccelerationMode)
{
	bAccelerationMode = bInAccelerationMode;
}

bool FAngularDriveConstraint::GetAccelerationMode() const
{
	return bAccelerationMode;
}

