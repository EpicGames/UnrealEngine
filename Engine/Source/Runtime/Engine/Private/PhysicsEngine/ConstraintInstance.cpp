// Copyright Epic Games, Inc. All Rights Reserved.

#include "PhysicsEngine/ConstraintInstance.h"

#include "Engine/SkeletalMesh.h"
#include "UObject/FrameworkObjectVersion.h"
#include "Components/SkeletalMeshComponent.h"
#include "PhysicsEngine/PhysicsConstraintComponent.h"
#include "PhysicsEngine/PhysicsObjectExternalInterface.h"
#include "PhysicsEngine/PhysicsAsset.h"
#include "PhysicsProxy/SingleParticlePhysicsProxy.h"

#include "Logging/MessageLog.h"
#include "Misc/UObjectToken.h"
#include "Physics/Experimental/PhysScene_Chaos.h"
#include "Chaos/PBDJointConstraintUtilities.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ConstraintInstance)

#if WITH_EDITOR
#include "UObject/UnrealType.h"
#endif

using namespace PhysicsInterfaceTypes;

#define LOCTEXT_NAMESPACE "ConstraintInstance"

TAutoConsoleVariable<float> CVarConstraintLinearDampingScale(
	TEXT("p.ConstraintLinearDampingScale"),
	1.f,
	TEXT("The multiplier of constraint linear damping in simulation. Default: 1"),
	ECVF_ReadOnly);

TAutoConsoleVariable<float> CVarConstraintLinearStiffnessScale(
	TEXT("p.ConstraintLinearStiffnessScale"),
	1.f,
	TEXT("The multiplier of constraint linear stiffness in simulation. Default: 1"),
	ECVF_ReadOnly);

TAutoConsoleVariable<float> CVarConstraintAngularDampingScale(
	TEXT("p.ConstraintAngularDampingScale"),
	100000.f,
	TEXT("The multiplier of constraint angular damping in simulation. Default: 100000"),
	ECVF_ReadOnly);

TAutoConsoleVariable<float> CVarConstraintAngularStiffnessScale(
	TEXT("p.ConstraintAngularStiffnessScale"),
	100000.f,
	TEXT("The multiplier of constraint angular stiffness in simulation. Default: 100000"),
	ECVF_ReadOnly);

bool bEnableSkeletalMeshConstraints = true;
FAutoConsoleVariableRef CVarEnableSkeletalMeshConstraints(TEXT("p.EnableSkeletalMeshConstraints"), bEnableSkeletalMeshConstraints, TEXT("Enable skeletal mesh constraints defined within the Physics Asset Editor"));

// warning : setting the value to false can have negative impact on ragdoll, if they start in a kinematic state and then have their bodies turned dynamic
bool bAllowKinematicKinematicConstraints = true;
FAutoConsoleVariableRef CVarAllowKinematicKinematicConstraints(TEXT("p.AllowKinematicKinematicConstraints"), bAllowKinematicKinematicConstraints, TEXT("Do not create constraints between two rigid kinematics."));

/** Handy macro for setting BIT of VAR based on the bool CONDITION */
#define SET_DRIVE_PARAM(VAR, CONDITION, BIT)   (VAR) = (CONDITION) ? ((VAR) | (BIT)) : ((VAR) & ~(BIT))

float RevolutionsToRads(const float Revolutions)
{
	return Revolutions * 2.f * UE_PI;
}

FVector RevolutionsToRads(const FVector Revolutions)
{
	return Revolutions * 2.f * UE_PI;
}

/** Returns the 'To' bone's transform relative to the 'From' bone. */
FTransform CalculateRelativeBoneTransform(const FName ToBoneName, const FName FromBoneName, const FReferenceSkeleton& ReferenceSkeleton)
{
	FTransform RelativeBoneTransform = FTransform::Identity;

	const TArray<FTransform>& LocalPose = ReferenceSkeleton.GetRefBonePose();
	int32 ToBoneAncestorIndex = ReferenceSkeleton.FindBoneIndex(ToBoneName);
	int32 FromBoneAncestorIndex = ReferenceSkeleton.FindBoneIndex(FromBoneName);

	check(LocalPose.IsValidIndex(ToBoneAncestorIndex));
	check(LocalPose.IsValidIndex(FromBoneAncestorIndex));

	FTransform ToCommonBasisTransform = FTransform::Identity;
	FTransform FromCommonBasisTransform = FTransform::Identity;

	// Traverse the skeleton from child to parent bone, accumulating transforms until we have a transform for both bones relative to a common ancestor bone in the hierarchy.
	while (LocalPose.IsValidIndex(ToBoneAncestorIndex) && LocalPose.IsValidIndex(FromBoneAncestorIndex))
	{
		if (ToBoneAncestorIndex > FromBoneAncestorIndex)
		{
			ToCommonBasisTransform = ToCommonBasisTransform * LocalPose[ToBoneAncestorIndex];
			ToBoneAncestorIndex = ReferenceSkeleton.GetParentIndex(ToBoneAncestorIndex);
		}
		else if (FromBoneAncestorIndex > ToBoneAncestorIndex)
		{
			FromCommonBasisTransform = FromCommonBasisTransform * LocalPose[FromBoneAncestorIndex];
			FromBoneAncestorIndex = ReferenceSkeleton.GetParentIndex(FromBoneAncestorIndex);
		}
		else // FromBoneAncestorIndex == ToBoneAncestorIndex 
		{
			// Found a bone that exists in the hierarchy of both the 'To' and 'From' bones.
			break;
		}
	}

	check(ToBoneAncestorIndex == FromBoneAncestorIndex); // A pair of bones should always have at least one common bone in their hierarchies, even if it's the root bone.

	// Calculate the transform of the 'To' bone relative to the 'From' bone.
	if (ToBoneAncestorIndex == FromBoneAncestorIndex)
	{
		RelativeBoneTransform = ToCommonBasisTransform.GetRelativeTransform(FromCommonBasisTransform);
	}

	return RelativeBoneTransform;
}

#if WITH_EDITOR
void FConstraintProfileProperties::SyncChangedConstraintProperties(FPropertyChangedChainEvent& PropertyChangedEvent)
{
	static const FName StiffnessProperty = GET_MEMBER_NAME_CHECKED(FConstraintDrive, Stiffness);
	static const FName MaxForceName = GET_MEMBER_NAME_CHECKED(FConstraintDrive, MaxForce);
	static const FName DampingName = GET_MEMBER_NAME_CHECKED(FConstraintDrive, Damping);

	if (TDoubleLinkedList<FProperty*>::TDoubleLinkedListNode* PropertyNode = PropertyChangedEvent.PropertyChain.GetTail())
	{
		if (TDoubleLinkedList<FProperty*>::TDoubleLinkedListNode* ParentProeprtyNode = PropertyNode->GetPrevNode())
		{
			if (FProperty* Property = PropertyNode->GetValue())
			{
				if (FProperty* ParentProperty = ParentProeprtyNode->GetValue())
				{
					const FName PropertyName = Property->GetFName();
					const FName ParentPropertyName = ParentProperty->GetFName();

					if (ParentPropertyName == GET_MEMBER_NAME_CHECKED(FLinearDriveConstraint, XDrive))
					{
						if (StiffnessProperty == PropertyName)
						{
							LinearDrive.YDrive.Stiffness = LinearDrive.XDrive.Stiffness;
							LinearDrive.ZDrive.Stiffness = LinearDrive.XDrive.Stiffness;
						}
						else if (MaxForceName == PropertyName)
						{
							LinearDrive.YDrive.MaxForce = LinearDrive.XDrive.MaxForce;
							LinearDrive.ZDrive.MaxForce = LinearDrive.XDrive.MaxForce;
						}
						else if (DampingName == PropertyName)
						{
							LinearDrive.YDrive.Damping = LinearDrive.XDrive.Damping;
							LinearDrive.ZDrive.Damping = LinearDrive.XDrive.Damping;
						}
					}
					else if (ParentPropertyName == GET_MEMBER_NAME_CHECKED(FAngularDriveConstraint, SlerpDrive))
					{
						if (StiffnessProperty == PropertyName)
						{
							AngularDrive.SwingDrive.Stiffness = AngularDrive.SlerpDrive.Stiffness;
							AngularDrive.TwistDrive.Stiffness = AngularDrive.SlerpDrive.Stiffness;
						}
						else if (MaxForceName == PropertyName)
						{
							AngularDrive.SwingDrive.MaxForce = AngularDrive.SlerpDrive.MaxForce;
							AngularDrive.TwistDrive.MaxForce = AngularDrive.SlerpDrive.MaxForce;
						}
						else if (DampingName == PropertyName)
						{
							AngularDrive.SwingDrive.Damping = AngularDrive.SlerpDrive.Damping;
							AngularDrive.TwistDrive.Damping = AngularDrive.SlerpDrive.Damping;
						}
					}
				}
			}
		}
	}
}
#endif

FConstraintProfileProperties::FConstraintProfileProperties()
	: ProjectionLinearTolerance(5.f)
	, ProjectionAngularTolerance(180.f)
	, ProjectionLinearAlpha(1.0f)
	, ProjectionAngularAlpha(0.0f)
	, ShockPropagationAlpha(0.3f)
	, LinearBreakThreshold(300.f)
	, LinearPlasticityThreshold(0.1f)
	, AngularBreakThreshold(500.f)
	, AngularPlasticityThreshold(10.f)
	, ContactTransferScale(0.f)
	, bDisableCollision(false)
	, bParentDominates(false)
	, bEnableShockPropagation(false)
	, bEnableProjection(true)
	, bEnableMassConditioning(true)
	, bUseLinearJointSolver(true)
	, bAngularBreakable(false)
	, bAngularPlasticity(false)
	, bLinearBreakable(false)
	, bLinearPlasticity(false)
	, LinearPlasticityType(EConstraintPlasticityType::CCPT_Free)
{
}

void FConstraintInstance::UpdateLinearLimit()
{
	FPhysicsInterface::ExecuteOnUnbrokenConstraintReadWrite(ConstraintHandle, [&](const FPhysicsConstraintHandle& InUnbrokenConstraint)
	{
		ProfileInstance.LinearLimit.UpdateLinearLimit_AssumesLocked(InUnbrokenConstraint, AverageMass, bScaleLinearLimits ? LastKnownScale : 1.0f);
	});
}

void FConstraintInstance::UpdateAngularLimit()
{
	FPhysicsInterface::ExecuteOnUnbrokenConstraintReadWrite(ConstraintHandle, [&](const FPhysicsConstraintHandle& InUnbrokenConstraint)
	{
		ProfileInstance.ConeLimit.UpdateConeLimit_AssumesLocked(InUnbrokenConstraint, AverageMass);
		ProfileInstance.TwistLimit.UpdateTwistLimit_AssumesLocked(InUnbrokenConstraint, AverageMass);
	});
}

void FConstraintInstance::UpdateBreakable()
{
	FPhysicsInterface::ExecuteOnUnbrokenConstraintReadWrite(ConstraintHandle, [&](const FPhysicsConstraintHandle& InUnbrokenConstraint)
	{
		ProfileInstance.UpdateBreakable_AssumesLocked(InUnbrokenConstraint);
	});
}

void FConstraintProfileProperties::UpdateBreakable_AssumesLocked(const FPhysicsConstraintHandle& InConstraintRef) const
{
	const float LinearBreakForce = bLinearBreakable ? LinearBreakThreshold : UE_MAX_FLT;
	const float AngularBreakForce = bAngularBreakable ? AngularBreakThreshold : UE_MAX_FLT;

	FPhysicsInterface::SetBreakForces_AssumesLocked(InConstraintRef, LinearBreakForce, AngularBreakForce);
}

void FConstraintInstance::UpdatePlasticity()
{
	FPhysicsInterface::ExecuteOnUnbrokenConstraintReadWrite(ConstraintHandle, [&](const FPhysicsConstraintHandle& InUnbrokenConstraint)
		{
			ProfileInstance.UpdatePlasticity_AssumesLocked(InUnbrokenConstraint);
		});
}

void FConstraintInstance::UpdateContactTransferScale()
{
	FPhysicsInterface::ExecuteOnUnbrokenConstraintReadWrite(ConstraintHandle, [&](const FPhysicsConstraintHandle& InConstraint)
		{
			ProfileInstance.UpdateContactTransferScale_AssumesLocked(InConstraint);
		});
}

void FConstraintProfileProperties::UpdatePlasticity_AssumesLocked(const FPhysicsConstraintHandle& InConstraintRef) const
{
	const float LinearPlasticityLimit = bLinearPlasticity ? LinearPlasticityThreshold : FLT_MAX;
	const float AngularPlasticityLimit = bAngularPlasticity ? FMath::DegreesToRadians(AngularPlasticityThreshold) : UE_MAX_FLT;

	FPhysicsInterface::SetPlasticityLimits_AssumesLocked(InConstraintRef, LinearPlasticityLimit, AngularPlasticityLimit, LinearPlasticityType);
}

void FConstraintProfileProperties::UpdateContactTransferScale_AssumesLocked(const FPhysicsConstraintHandle& InConstraintRef) const
{
	FPhysicsInterface::SetContactTransferScale_AssumesLocked(InConstraintRef, ContactTransferScale);
}

void FConstraintInstance::UpdateDriveTarget()
{
	FPhysicsInterface::ExecuteOnUnbrokenConstraintReadWrite(ConstraintHandle, [&](const FPhysicsConstraintHandle& InUnbrokenConstraint)
	{
		FPhysicsInterface::UpdateDriveTarget_AssumesLocked(InUnbrokenConstraint, ProfileInstance.LinearDrive, ProfileInstance.AngularDrive);
	});
}

#if WITH_EDITORONLY_DATA

FTransform FConstraintInstance::CalculateDefaultParentTransform(const UPhysicsAsset* const PhysicsAsset) const
{
	if (PhysicsAsset)
	{
		if (USkeletalMesh* const PreviewSkelMesh = PhysicsAsset->GetPreviewMesh())
		{
			return CalculateRelativeBoneTransform(ConstraintBone1, ConstraintBone2, PreviewSkelMesh->GetRefSkeleton());
		}
	}

	return FTransform::Identity;
}

FTransform FConstraintInstance::CalculateDefaultChildTransform() const
{
	return FTransform::Identity;
} 

void FConstraintInstance::SnapTransformsToDefault(const EConstraintTransformComponentFlags SnapFlags, const UPhysicsAsset* const PhysicsAsset)
{
	const FTransform ParentTransform = CalculateDefaultParentTransform(PhysicsAsset);
	const FTransform ChildTransform = CalculateDefaultChildTransform();

	if (EnumHasAnyFlags(SnapFlags, EConstraintTransformComponentFlags::ChildPosition))
	{
		SetRefPosition(EConstraintFrame::Frame1, ChildTransform.GetLocation());
	}

	if (EnumHasAnyFlags(SnapFlags, EConstraintTransformComponentFlags::ChildRotation))
	{
		SetRefOrientation(EConstraintFrame::Frame1, ChildTransform.GetUnitAxis(EAxis::X), ChildTransform.GetUnitAxis(EAxis::Y));
	}

	if (EnumHasAnyFlags(SnapFlags, EConstraintTransformComponentFlags::ParentPosition))
	{
		SetRefPosition(EConstraintFrame::Frame2, ParentTransform.GetLocation());
	}

	if (EnumHasAnyFlags(SnapFlags, EConstraintTransformComponentFlags::ParentRotation))
	{
		SetRefOrientation(EConstraintFrame::Frame2, ParentTransform.GetUnitAxis(EAxis::X), ParentTransform.GetUnitAxis(EAxis::Y));
	}
}

#endif // WITH_EDITORONLY_DATA

/** Constructor **/
FConstraintInstanceBase::FConstraintInstanceBase()
{
	Reset();
}

void FConstraintInstanceBase::Reset()
{
	ConstraintIndex = 0;
	ConstraintHandle.Reset();
	PhysScene = nullptr;
}

void FConstraintInstanceBase::SetConstraintBrokenDelegate(FOnConstraintBroken InConstraintBrokenDelegate)
{
	OnConstraintBrokenDelegate = InConstraintBrokenDelegate;
}

void FConstraintInstanceBase::SetConstraintViolatedDelegate(FOnConstraintViolated InConstraintViolatedDelegate)
{
	OnConstraintViolatedDelegate = InConstraintViolatedDelegate;
}


/** Constructor **/
FConstraintInstance::FConstraintInstance()
	: FConstraintInstanceBase()
	, LastKnownScale(1.f)
	, AngularRotationOffset(ForceInitToZero)
	, bScaleLinearLimits(true)
	, AverageMass(0.f)
	, UserData(this)
#if WITH_EDITORONLY_DATA
	, bDisableCollision_DEPRECATED(false)
	, bEnableProjection_DEPRECATED(true)
	, ProjectionLinearTolerance_DEPRECATED(5.f)
	, ProjectionAngularTolerance_DEPRECATED(180.f)
	, LinearXMotion_DEPRECATED(ACM_Locked)
	, LinearYMotion_DEPRECATED(ACM_Locked)
	, LinearZMotion_DEPRECATED(ACM_Locked)
	, LinearLimitSize_DEPRECATED(0.f)
	, bLinearLimitSoft_DEPRECATED(false)
	, LinearLimitStiffness_DEPRECATED(0.f)
	, LinearLimitDamping_DEPRECATED(0.f)
	, bLinearBreakable_DEPRECATED(false)
	, LinearBreakThreshold_DEPRECATED(300.f)
	, AngularSwing1Motion_DEPRECATED(ACM_Free)
	, AngularTwistMotion_DEPRECATED(ACM_Free)
	, AngularSwing2Motion_DEPRECATED(ACM_Free)
	, bSwingLimitSoft_DEPRECATED(true)
	, bTwistLimitSoft_DEPRECATED(true)
	, Swing1LimitAngle_DEPRECATED(45)
	, TwistLimitAngle_DEPRECATED(45)
	, Swing2LimitAngle_DEPRECATED(45)
	, SwingLimitStiffness_DEPRECATED(50)
	, SwingLimitDamping_DEPRECATED(5)
	, TwistLimitStiffness_DEPRECATED(50)
	, TwistLimitDamping_DEPRECATED(5)
	, bAngularBreakable_DEPRECATED(false)
	, AngularBreakThreshold_DEPRECATED(500.f)
	, bLinearXPositionDrive_DEPRECATED(false)
	, bLinearXVelocityDrive_DEPRECATED(false)
	, bLinearYPositionDrive_DEPRECATED(false)
	, bLinearYVelocityDrive_DEPRECATED(false)
	, bLinearZPositionDrive_DEPRECATED(false)
	, bLinearZVelocityDrive_DEPRECATED(false)
	, bLinearPositionDrive_DEPRECATED(false)
	, bLinearVelocityDrive_DEPRECATED(false)
	, LinearPositionTarget_DEPRECATED(ForceInit)
	, LinearVelocityTarget_DEPRECATED(ForceInit)
	, LinearDriveSpring_DEPRECATED(50.0f)
	, LinearDriveDamping_DEPRECATED(1.0f)
	, LinearDriveForceLimit_DEPRECATED(0)
	, bSwingPositionDrive_DEPRECATED(false)
	, bSwingVelocityDrive_DEPRECATED(false)
	, bTwistPositionDrive_DEPRECATED(false)
	, bTwistVelocityDrive_DEPRECATED(false)
	, bAngularSlerpDrive_DEPRECATED(false)
	, bAngularOrientationDrive_DEPRECATED(false)
	, bEnableSwingDrive_DEPRECATED(true)
	, bEnableTwistDrive_DEPRECATED(true)
	, bAngularVelocityDrive_DEPRECATED(false)
	, AngularPositionTarget_DEPRECATED(ForceInit)
	, AngularDriveMode_DEPRECATED(EAngularDriveMode::SLERP)
	, AngularOrientationTarget_DEPRECATED(ForceInit)
	, AngularVelocityTarget_DEPRECATED(ForceInit)
	, AngularDriveSpring_DEPRECATED(50.0f)
	, AngularDriveDamping_DEPRECATED(1.0f)
	, AngularDriveForceLimit_DEPRECATED(0)
#endif	//EDITOR_ONLY_DATA
{
	Pos1 = FVector(0.0f, 0.0f, 0.0f);
	PriAxis1 = FVector(1.0f, 0.0f, 0.0f);
	SecAxis1 = FVector(0.0f, 1.0f, 0.0f);

	Pos2 = FVector(0.0f, 0.0f, 0.0f);
	PriAxis2 = FVector(1.0f, 0.0f, 0.0f);
	SecAxis2 = FVector(0.0f, 1.0f, 0.0f);
}

void FConstraintInstance::SetDisableCollision(bool InDisableCollision)
{
	ProfileInstance.bDisableCollision = InDisableCollision;

	FPhysicsInterface::ExecuteOnUnbrokenConstraintReadWrite(ConstraintHandle, [&](const FPhysicsConstraintHandle& InUnbrokenConstraint)
	{
		FPhysicsInterface::SetCollisionEnabled(InUnbrokenConstraint, !InDisableCollision);
	});
}

float ComputeAverageMass_AssumesLocked(Chaos::FPhysicsObject* Body1, Chaos::FPhysicsObject* Body2)
{
	float AverageMass = 0;
	int NumDynamic = 0;

	Chaos::FReadPhysicsObjectInterface_External Interface = FPhysicsObjectExternalInterface::GetRead_AssumesLocked();

	if (Interface.AreAllRigidBody({ &Body1, 1 }))
	{
		AverageMass += Interface.GetMass({ &Body1, 1 });
		++NumDynamic;
	}

	if (Interface.AreAllRigidBody({ &Body2, 1 }))
	{
		AverageMass += Interface.GetMass({ &Body2, 1 });
		++NumDynamic;
	}

	if(NumDynamic > 1)
	{
		AverageMass = AverageMass / NumDynamic;
	}

	return AverageMass;
}

bool GetActorRefs(FBodyInstance* Body1, FBodyInstance* Body2, FPhysicsActorHandle& OutActorRef1, FPhysicsActorHandle& OutActorRef2, UObject* DebugOwner)
{
	FPhysicsActorHandle ActorRef1 = Body1 ? Body1->GetPhysicsActor() : FPhysicsActorHandle();
	FPhysicsActorHandle ActorRef2 = Body2 ? Body2->GetPhysicsActor() : FPhysicsActorHandle();

	// Do not create joint unless you have two actors
	// Do not create joint unless one of the actors is dynamic
	bool bActor1InvalidRigidBody = false;
	bool bActor2InvalidRigidBody = false;
	FPhysicsCommand::ExecuteRead(ActorRef1, ActorRef2, [&bActor1InvalidRigidBody, &bActor2InvalidRigidBody, ActorRef1, ActorRef2](const FPhysicsActorHandle& ActorA, const FPhysicsActorHandle& ActorB)
	{
		bActor1InvalidRigidBody = !FPhysicsInterface::IsValid(ActorRef1) || !FPhysicsInterface::IsRigidBody(ActorRef1);
		bActor2InvalidRigidBody = !FPhysicsInterface::IsValid(ActorRef2) || !FPhysicsInterface::IsRigidBody(ActorRef2);
	});
	if(bActor1InvalidRigidBody && bActor2InvalidRigidBody)
	{
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
		FMessageLog("PIE").Warning()
			->AddToken(FTextToken::Create(LOCTEXT("TwoStaticBodiesWarningStart", "Constraint in")))
			->AddToken(FTextToken::Create(FText::Format(LOCTEXT("TwoStaticBodiesWarningOwner", "'{0}'"), FText::FromString(GetPathNameSafe(DebugOwner)))))
			->AddToken(FTextToken::Create(LOCTEXT("TwoStaticBodiesWarningEnd", "attempting to create a joint between objects that are both static.  No joint created.")));
#endif // !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
		return false;
	}

	if(FPhysicsInterface::IsValid(ActorRef1) && FPhysicsInterface::IsValid(ActorRef2) && ActorRef1 == ActorRef2)
	{
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
		const UPrimitiveComponent* PrimComp = Body1 ? Body1->OwnerComponent.Get() : nullptr;
		FMessageLog("PIE").Warning()
			->AddToken(FTextToken::Create(LOCTEXT("SameBodyWarningStart", "Constraint in")))
			->AddToken(FTextToken::Create(FText::Format(LOCTEXT("SameBodyWarningOwner", "'{0}'"), FText::FromString(GetPathNameSafe(DebugOwner)))))
			->AddToken(FTextToken::Create(LOCTEXT("SameBodyWarningMid", "attempting to create a joint to the same body")))
			->AddToken(FUObjectToken::Create(PrimComp));
#endif // !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
		return false;
	}

	// Ensure that actors are either invalid (ie 'world') or valid to simulate.
	bool bActor1ValidToSim = false;
	bool bActor2ValidToSim = false;
	FPhysicsCommand::ExecuteRead(ActorRef1, ActorRef2, [&](const FPhysicsActorHandle& ActorA, const FPhysicsActorHandle& ActorB)
	{
		bActor1ValidToSim = !FPhysicsInterface::IsValid(ActorRef1) || FPhysicsInterface::CanSimulate_AssumesLocked(ActorRef1);
		bActor2ValidToSim = !FPhysicsInterface::IsValid(ActorRef2) || FPhysicsInterface::CanSimulate_AssumesLocked(ActorRef2);
	});

	if(!bActor1ValidToSim || !bActor2ValidToSim)
	{
		OutActorRef1 = FPhysicsActorHandle();
		OutActorRef2 = FPhysicsActorHandle();

		return false;
	}

	OutActorRef1 = ActorRef1;
	OutActorRef2 = ActorRef2;

	return true;
}

bool FConstraintInstance::CreateJoint_AssumesLocked(Chaos::FPhysicsObject* Body1, Chaos::FPhysicsObject* Body2)
{
	LLM_SCOPE(ELLMTag::ChaosConstraint);

	Chaos::FReadPhysicsObjectInterface_External Interface = FPhysicsObjectExternalInterface::GetRead_AssumesLocked();

	FTransform Local1 = GetRefFrame(EConstraintFrame::Frame1);
	if (Interface.AreAllValid({ &Body1, 1 }))
	{
		Local1.ScaleTranslation(FVector(LastKnownScale));
	}

	checkf(Local1.IsValid() && !Local1.ContainsNaN(), TEXT("%s"), *Local1.ToString());

	FTransform Local2 = GetRefFrame(EConstraintFrame::Frame2);
	if (Interface.AreAllValid({ &Body2, 1 }))
	{
		Local2.ScaleTranslation(FVector(LastKnownScale));
	}
	
	checkf(Local2.IsValid() && !Local2.ContainsNaN(), TEXT("%s"), *Local2.ToString());

	if (bEnableSkeletalMeshConstraints)
	{
		ConstraintHandle = FPhysicsInterface::CreateConstraint(Body1, Body2, Local1, Local2);
	}
	if(!ConstraintHandle.IsValid())
	{
		UE_LOGF(LogPhysics, Log, "FConstraintInstance::CreatePxJoint_AssumesLocked - Invalid 6DOF joint (%ls)", *JointName.ToString());
		return false;
	}

	FPhysicsInterface::SetConstraintUserData(ConstraintHandle, &UserData);

	return true;
}

void FConstraintProfileProperties::UpdateConstraintFlags_AssumesLocked(const FPhysicsConstraintHandle& InConstraintRef) const
{
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	FPhysicsInterface::SetCanVisualize(InConstraintRef, true);
#endif

	FPhysicsInterface::SetCollisionEnabled(InConstraintRef, !bDisableCollision);
	FPhysicsInterface::SetProjectionEnabled_AssumesLocked(InConstraintRef, bEnableProjection, ProjectionLinearAlpha, ProjectionAngularAlpha, ProjectionLinearTolerance, ProjectionAngularTolerance);
	FPhysicsInterface::SetShockPropagationEnabled_AssumesLocked(InConstraintRef, bEnableShockPropagation, ShockPropagationAlpha);
	FPhysicsInterface::SetParentDominates_AssumesLocked(InConstraintRef, bParentDominates);
	FPhysicsInterface::SetMassConditioningEnabled_AssumesLocked(InConstraintRef, bEnableMassConditioning);
	FPhysicsInterface::SetUseLinearJointSolver_AssumesLocked(InConstraintRef, bUseLinearJointSolver);
}


void FConstraintInstance::UpdateAverageMass_AssumesLocked(Chaos::FPhysicsObject* Body1, Chaos::FPhysicsObject* Body2)
{
	// @todo(chaos): Average mass isn't required by anything any more. We should probably remove this
	AverageMass = ComputeAverageMass_AssumesLocked(Body1, Body2);
}

/** 
 *	Create physics engine constraint.
 */
void FConstraintInstance::InitConstraint(FBodyInstance* Body1, FBodyInstance* Body2, float InScale, UObject* DebugOwner, FOnConstraintBroken InConstraintBrokenDelegate, FOnPlasticDeformation InPlasticDeformationDelegate)
{
	FPhysicsActorHandle Actor1;
	FPhysicsActorHandle Actor2;

	const bool bValidActors = GetActorRefs(Body1, Body2, Actor1, Actor2, DebugOwner);
	if (!bValidActors)
	{
		return;
	}

	InitConstraint(Actor1 ? Actor1->GetPhysicsObject() : nullptr, Actor2 ? Actor2->GetPhysicsObject() : nullptr, InScale, DebugOwner, InConstraintBrokenDelegate, InPlasticDeformationDelegate);
}

void FConstraintInstance::InitConstraint(Chaos::FPhysicsObject* Body1, Chaos::FPhysicsObject* Body2, float InScale, UObject* DebugOwner, FOnConstraintBroken InConstraintBrokenDelegate, FOnPlasticDeformation InPlasticDeformationDelegate)
{
	{
		Chaos::FPhysicsObject* Bodies[2] = { Body1, Body2 };
		FLockedReadPhysicsObjectExternalInterface Interface = FPhysicsObjectExternalInterface::LockRead({ Bodies, 2 });

		const bool bBody1Valid = Interface->AreAllValid({ &Body1, 1 });
		const bool bBody2Valid = Interface->AreAllValid({ &Body2, 1 });
		if (!bAllowKinematicKinematicConstraints && (!bBody1Valid || Interface->AreAllKinematic({ &Body1, 1 })) && (!bBody2Valid || Interface->AreAllKinematic({ &Body2, 1 })))
		{
			return;
		}
	}

	FPhysicsCommand::ExecuteWrite(Body1, Body2, [&](Chaos::FPhysicsObject* ActorA, Chaos::FPhysicsObject* ActorB)
	{
		InitConstraint_AssumesLocked(ActorA, ActorB, InScale, InConstraintBrokenDelegate, InPlasticDeformationDelegate);
	});
}

void FConstraintInstance::InitConstraint_AssumesLocked(const FPhysicsActorHandle& ActorRef1, const FPhysicsActorHandle& ActorRef2, float InScale, FOnConstraintBroken InConstraintBrokenDelegate, FOnPlasticDeformation InPlasticDeformationDelegate)
{
	Chaos::FPhysicsObject* Body1 = ActorRef1 ? ActorRef1->GetPhysicsObject() : nullptr;
	Chaos::FPhysicsObject* Body2 = ActorRef2 ? ActorRef2->GetPhysicsObject() : nullptr;
	InitConstraint_AssumesLocked(Body1, Body2, InScale, InConstraintBrokenDelegate, InPlasticDeformationDelegate);
}

void FConstraintInstance::InitConstraint_AssumesLocked(Chaos::FPhysicsObject* Body1, Chaos::FPhysicsObject* Body2, float InScale, FOnConstraintBroken InConstraintBrokenDelegate, FOnPlasticDeformation InPlasticDeformationDelegate)
{
	OnConstraintBrokenDelegate = InConstraintBrokenDelegate;
	OnPlasticDeformationDelegate = InPlasticDeformationDelegate;
	LastKnownScale = InScale;

	UserData = FChaosUserData(this);

	Chaos::FWritePhysicsObjectInterface_External Interface = FPhysicsObjectExternalInterface::GetWrite_AssumesLocked();
	// Creating/Destroying a joint between two bodies will wake them, so we may want to re-sleep them
	const bool bActor1WasAsleep = Interface.AreAllValid({ &Body1, 1 }) && Interface.AreAllSleeping({ &Body1, 1 });
	const bool bActor2WasAsleep = Interface.AreAllValid({ &Body2, 1 }) && Interface.AreAllSleeping({ &Body2, 1 });

	// if there's already a constraint, get rid of it first
	if (ConstraintHandle.IsValid())
	{
		TermConstraint();
	}

	if (!CreateJoint_AssumesLocked(Body1, Body2))
	{
		return;
	}

	// update mass
	UpdateAverageMass_AssumesLocked(Body1, Body2);

	ProfileInstance.Update_AssumesLocked(ConstraintHandle, AverageMass, bScaleLinearLimits ? LastKnownScale : 1.f, true);

	// Put the bodies back to sleep both bodies were asleep
	if (bActor1WasAsleep && bActor2WasAsleep)
	{
		if (!Interface.AreAllKinematic({ &Body1, 1 }))
		{
			Interface.PutToSleep({ &Body1, 1 });
		}

		if (!Interface.AreAllKinematic({ &Body2, 1 }))
		{
			Interface.PutToSleep({ &Body2, 1 });
		}
	}
}

void FConstraintProfileProperties::Update_AssumesLocked(const FPhysicsConstraintHandle& InConstraintRef, float AverageMass, float UseScale, bool InInitialize) const
{
	// flags and projection settings
	UpdateConstraintFlags_AssumesLocked(InConstraintRef);

	//limits
	LinearLimit.UpdateLinearLimit_AssumesLocked(InConstraintRef, AverageMass, UseScale);
	ConeLimit.UpdateConeLimit_AssumesLocked(InConstraintRef, AverageMass);
	TwistLimit.UpdateTwistLimit_AssumesLocked(InConstraintRef, AverageMass);

	UpdateBreakable_AssumesLocked(InConstraintRef);
	UpdatePlasticity_AssumesLocked(InConstraintRef);
	UpdateContactTransferScale_AssumesLocked(InConstraintRef);

	// Target
	FPhysicsInterface::UpdateDriveTarget_AssumesLocked(InConstraintRef, LinearDrive, AngularDrive, InInitialize);
}

void FConstraintInstance::TermConstraint()
{
	if (!ConstraintHandle.IsValid())
	{
		return;
	}

	FPhysicsConstraintHandle PhysConstraint = GetPhysicsConstraintRef();

	FPhysicsCommand::ExecuteWrite(PhysConstraint, [&](const FPhysicsConstraintHandle& Constraint)
	{
		FPhysicsInterface::ReleaseConstraint(ConstraintHandle);
	});
}

bool FConstraintInstance::IsTerminated() const
{
	return !ConstraintHandle.IsValid();
}

bool FConstraintInstance::IsValidConstraintInstance() const
{
	return ConstraintHandle.IsValid();
}

void FConstraintInstance::CopyProfilePropertiesFrom(const FConstraintProfileProperties& FromProperties)
{
	ProfileInstance = FromProperties;

	FPhysicsInterface::ExecuteOnUnbrokenConstraintReadWrite(ConstraintHandle, [&](const FPhysicsConstraintHandle& InUnbrokenConstraint)
	{
		ProfileInstance.Update_AssumesLocked(ConstraintHandle, AverageMass, bScaleLinearLimits ? LastKnownScale : 1.0f);
	});
}

const FPhysicsConstraintHandle& FConstraintInstance::GetPhysicsConstraintRef() const
{
	return ConstraintHandle;
}

void FConstraintInstance::SerializeState(Chaos::FSerializedDataBuffer& OutSerializedState, Chaos::ESerializedDataContext DataContext)
{
	const EPhysicsInterfaceScopedThreadContext LockContext = DataContext == Chaos::ESerializedDataContext::Internal ? EPhysicsInterfaceScopedThreadContext::Internal : EPhysicsInterfaceScopedThreadContext::External;
	FPhysicsCommand::ExecuteRead(ConstraintHandle, [&OutSerializedState, DataContext](const FPhysicsConstraintHandle& Constraint)
	{
		FChaosEngineInterface::SerializeConstraintStateToBuffer(Constraint, OutSerializedState, DataContext);
	}, LockContext);
}

void FConstraintInstance::ApplySerializedState(Chaos::FSerializedDataBufferPtr&& InSerializedState, Chaos::ESerializedDataContext DataContext)
{
	const EPhysicsInterfaceScopedThreadContext LockContext = DataContext == Chaos::ESerializedDataContext::Internal ? EPhysicsInterfaceScopedThreadContext::Internal : EPhysicsInterfaceScopedThreadContext::External;
	FPhysicsCommand::ExecuteWrite(ConstraintHandle, [&InSerializedState, this, DataContext](const FPhysicsConstraintHandle& Constraint)
	{
		FChaosEngineInterface::ApplySerializedStateToConstraint(Constraint, MoveTemp(InSerializedState), DataContext);
	}, LockContext, EPhysicsInterfaceScopedTransactionMode::Normal);
}

void FConstraintInstance::CopyConstraintGeometryFrom(const FConstraintInstance* FromInstance)
{
	Pos1 = FromInstance->Pos1;
	PriAxis1 = FromInstance->PriAxis1;
	SecAxis1 = FromInstance->SecAxis1;

	Pos2 = FromInstance->Pos2;
	PriAxis2 = FromInstance->PriAxis2;
	SecAxis2 = FromInstance->SecAxis2;
}

void FConstraintInstance::CopyConstraintParamsFrom(const FConstraintInstance* FromInstance)
{
	check(FromInstance->IsTerminated());
	check(IsTerminated());
	check(FromInstance->PhysScene == nullptr);

	*this = *FromInstance;
}

void FConstraintInstance::CopyConstraintPhysicalPropertiesFrom(const FConstraintInstance* FromInstance, bool bKeepPosition, bool bKeepRotation)
{
	check(FromInstance);

	FConstraintInstance OldInstance = *this;
	CopyConstraintParamsFrom(FromInstance);

	// Recover internal data we'd like to keep - i.e. bone indices, etc.
	ConstraintIndex = OldInstance.ConstraintIndex;
	ConstraintHandle = OldInstance.ConstraintHandle;
	JointName = OldInstance.JointName;
	ConstraintBone1 = OldInstance.ConstraintBone1;
	ConstraintBone2 = OldInstance.ConstraintBone2;

	if (bKeepPosition)
	{
		Pos1 = OldInstance.Pos1;
		Pos2 = OldInstance.Pos2;
	}

	if (bKeepRotation)
	{
		PriAxis1 = OldInstance.PriAxis1;
		SecAxis1 = OldInstance.SecAxis1;
		PriAxis2 = OldInstance.PriAxis2;
		SecAxis2 = OldInstance.SecAxis2;
		AngularRotationOffset = OldInstance.AngularRotationOffset;
	}
}

FTransform FConstraintInstance::GetRefFrame(EConstraintFrame::Type Frame) const
{
	FTransform Result;

	if(Frame == EConstraintFrame::Frame1)
	{
		Result = FTransform(PriAxis1, SecAxis1, PriAxis1 ^ SecAxis1, Pos1);
	}
	else
	{
		Result = FTransform(PriAxis2, SecAxis2, PriAxis2 ^ SecAxis2, Pos2);
	}

	float Error = FMath::Abs( Result.GetDeterminant() - 1.0f );
	if(Error > 0.01f)
	{
		UE_LOGF(LogPhysics, Warning,  "FConstraintInstance::GetRefFrame : Contained scale.");
	}

	return Result;
}

void FConstraintInstance::SetRefFrame(EConstraintFrame::Type Frame, const FTransform& RefFrame)
{
	if(Frame == EConstraintFrame::Frame1)
	{
		Pos1 = RefFrame.GetTranslation();
		PriAxis1 = RefFrame.GetUnitAxis( EAxis::X );
		SecAxis1 = RefFrame.GetUnitAxis( EAxis::Y );
	}
	else
	{
		Pos2 = RefFrame.GetTranslation();
		PriAxis2 = RefFrame.GetUnitAxis( EAxis::X );
		SecAxis2 = RefFrame.GetUnitAxis( EAxis::Y );
	}

	FPhysicsInterface::ExecuteOnUnbrokenConstraintReadWrite(ConstraintHandle, [&](const FPhysicsConstraintHandle& InUnbrokenConstraint)
	{
		FPhysicsInterface::SetLocalPose(InUnbrokenConstraint, RefFrame, Frame);
	});
}

void FConstraintInstance::SetRefPosition(EConstraintFrame::Type Frame, const FVector& RefPosition)
{
	if (Frame == EConstraintFrame::Frame1)
	{
		Pos1 = RefPosition;
	}
	else
	{
		Pos2 = RefPosition;
	}

	FPhysicsInterface::ExecuteOnUnbrokenConstraintReadWrite(ConstraintHandle, [&](const FPhysicsConstraintHandle& InUnbrokenConstraint)
	{
		FTransform LocalPose = FPhysicsInterface::GetLocalPose(InUnbrokenConstraint, Frame);
		LocalPose.SetLocation(RefPosition);
		FPhysicsInterface::SetLocalPose(InUnbrokenConstraint, LocalPose, Frame);
	});
}

void FConstraintInstance::SetRefOrientation(EConstraintFrame::Type Frame, const FVector& PriAxis, const FVector& SecAxis)
{
	FVector RefPos;
		
	if (Frame == EConstraintFrame::Frame1)
	{
		RefPos = Pos1;
		PriAxis1 = PriAxis;
		SecAxis1 = SecAxis;
	}
	else
	{
		RefPos = Pos2;
		PriAxis2 = PriAxis;
		SecAxis2 = SecAxis;
	}

	FPhysicsInterface::ExecuteOnUnbrokenConstraintReadWrite(ConstraintHandle, [&](const FPhysicsConstraintHandle& InUnbrokenConstraint)
	{
		FTransform URefTransform = FTransform(PriAxis, SecAxis, PriAxis ^ SecAxis, RefPos);
		FPhysicsInterface::SetLocalPose(InUnbrokenConstraint, URefTransform, Frame);
	});
}

/** Get the position of this constraint in world space. */
FVector FConstraintInstance::GetConstraintLocation()
{
	return FPhysicsInterface::GetLocation(ConstraintHandle);
}

void FConstraintInstance::GetConstraintForce(FVector& OutLinearForce, FVector& OutAngularForce)
{
	OutLinearForce = FVector::ZeroVector;
	OutAngularForce = FVector::ZeroVector;
	FPhysicsInterface::GetForce(ConstraintHandle, OutLinearForce, OutAngularForce);
}

bool FConstraintInstance::IsBroken()
{
	return FPhysicsInterface::IsBroken(ConstraintHandle);
}

/** Function for turning linear position drive on and off. */
void FConstraintInstance::SetLinearPositionDrive(bool bEnableXDrive, bool bEnableYDrive, bool bEnableZDrive)
{
	ProfileInstance.LinearDrive.SetLinearPositionDrive(bEnableXDrive, bEnableYDrive, bEnableZDrive);

	FPhysicsInterface::ExecuteOnUnbrokenConstraintReadWrite(ConstraintHandle, [this](const FPhysicsConstraintHandle& InUnbrokenConstraint)
	{
		FPhysicsInterface::UpdateLinearDrive_AssumesLocked(InUnbrokenConstraint, ProfileInstance.LinearDrive);
	});
}

/** Function for turning linear velocity drive on and off. */
void FConstraintInstance::SetLinearVelocityDrive(bool bEnableXDrive, bool bEnableYDrive, bool bEnableZDrive)
{
	ProfileInstance.LinearDrive.SetLinearVelocityDrive(bEnableXDrive, bEnableYDrive, bEnableZDrive);

	FPhysicsInterface::ExecuteOnUnbrokenConstraintReadWrite(ConstraintHandle, [this](const FPhysicsConstraintHandle& InUnbrokenConstraint)
	{
		FPhysicsInterface::UpdateLinearDrive_AssumesLocked(InUnbrokenConstraint, ProfileInstance.LinearDrive);
	});
}

void FConstraintInstance::SetOrientationDriveTwistAndSwing(bool InEnableTwistDrive, bool InEnableSwingDrive)
{
	ProfileInstance.AngularDrive.SetOrientationDriveTwistAndSwing(InEnableTwistDrive, InEnableSwingDrive);

	FPhysicsInterface::ExecuteOnUnbrokenConstraintReadWrite(ConstraintHandle, [this](const FPhysicsConstraintHandle& InUnbrokenConstraint)
	{
		FPhysicsInterface::UpdateAngularDrive_AssumesLocked(InUnbrokenConstraint, ProfileInstance.AngularDrive);
	});
}

void FConstraintInstance::GetOrientationDriveTwistAndSwing(bool& bOutEnableTwistDrive, bool& bOutEnableSwingDrive)
{
	bOutEnableTwistDrive = ProfileInstance.AngularDrive.TwistDrive.bEnablePositionDrive;
	bOutEnableSwingDrive = ProfileInstance.AngularDrive.SwingDrive.bEnablePositionDrive;
}

void FConstraintInstance::SetOrientationDriveSLERP(bool InEnableSLERP)
{
	ProfileInstance.AngularDrive.SetOrientationDriveSLERP(InEnableSLERP);

	FPhysicsInterface::ExecuteOnUnbrokenConstraintReadWrite(ConstraintHandle, [this](const FPhysicsConstraintHandle& InUnbrokenConstraint)
	{
		FPhysicsInterface::UpdateAngularDrive_AssumesLocked(InUnbrokenConstraint, ProfileInstance.AngularDrive);
	});
}

/** Set which twist and swing angular velocity drives are enabled. Only applicable when Twist And Swing drive mode is used */
void FConstraintInstance::SetAngularVelocityDriveTwistAndSwing(bool bInEnableTwistDrive, bool bInEnableSwingDrive)
{
	ProfileInstance.AngularDrive.SetAngularVelocityDriveTwistAndSwing(bInEnableTwistDrive, bInEnableSwingDrive);

	FPhysicsInterface::ExecuteOnUnbrokenConstraintReadWrite(ConstraintHandle, [this](const FPhysicsConstraintHandle& InUnbrokenConstraint)
	{
		FPhysicsInterface::UpdateAngularDrive_AssumesLocked(InUnbrokenConstraint, ProfileInstance.AngularDrive);
	});
}

void FConstraintInstance::GetAngularVelocityDriveTwistAndSwing(bool& bOutEnableTwistDrive, bool& bOutEnableSwingDrive)
{
	bOutEnableTwistDrive = ProfileInstance.AngularDrive.TwistDrive.bEnableVelocityDrive;
	bOutEnableSwingDrive = ProfileInstance.AngularDrive.SwingDrive.bEnableVelocityDrive;	
}

/** Set whether the SLERP angular velocity drive is enabled. Only applicable when SLERP drive mode is used */
void FConstraintInstance::SetAngularVelocityDriveSLERP(bool bInEnableSLERP)
{
	ProfileInstance.AngularDrive.SetAngularVelocityDriveSLERP(bInEnableSLERP);

	FPhysicsInterface::ExecuteOnUnbrokenConstraintReadWrite(ConstraintHandle, [this](const FPhysicsConstraintHandle& InUnbrokenConstraint)
	{
		FPhysicsInterface::UpdateAngularDrive_AssumesLocked(InUnbrokenConstraint, ProfileInstance.AngularDrive);
	});
}

/** Set the angular drive mode */
void FConstraintInstance::SetAngularDriveMode(EAngularDriveMode::Type DriveMode)
{
	ProfileInstance.AngularDrive.SetAngularDriveMode(DriveMode);

	FPhysicsInterface::ExecuteOnUnbrokenConstraintReadWrite(ConstraintHandle, [this](const FPhysicsConstraintHandle& InUnbrokenConstraint)
	{
		FPhysicsInterface::UpdateAngularDrive_AssumesLocked(InUnbrokenConstraint, ProfileInstance.AngularDrive);
	});
}

/** Function for setting linear position target. */
void FConstraintInstance::SetLinearPositionTarget(const FVector& InPosTarget)
{
	// If settings are the same, don't do anything.
	if( ProfileInstance.LinearDrive.PositionTarget == InPosTarget )
	{
		return;
	}

	ProfileInstance.LinearDrive.PositionTarget = InPosTarget;
	FPhysicsInterface::SetDrivePosition(ConstraintHandle, InPosTarget);
}

/** Function for setting linear velocity target. */
void FConstraintInstance::SetLinearVelocityTarget(const FVector& InVelTarget)
{
	// If settings are the same, don't do anything.
	if (ProfileInstance.LinearDrive.VelocityTarget == InVelTarget)
	{
		return;
	}

	ProfileInstance.LinearDrive.VelocityTarget = InVelTarget;
	FPhysicsInterface::SetDriveLinearVelocity(ConstraintHandle, InVelTarget);
}

/** Function for setting linear motor parameters. */
void FConstraintInstance::SetLinearDriveParams(float InSpring, float InDamping, float InForceLimit)
{
	ProfileInstance.LinearDrive.SetDriveParams(InSpring, InDamping, InForceLimit);

	FPhysicsInterface::ExecuteOnUnbrokenConstraintReadWrite(ConstraintHandle, [this](const FPhysicsConstraintHandle& InUnbrokenConstraint)
	{
		FPhysicsInterface::UpdateLinearDrive_AssumesLocked(InUnbrokenConstraint, ProfileInstance.LinearDrive);
	});
}

/** Function for setting linear motor parameters. */
void FConstraintInstance::SetLinearDriveParams(const FVector& InSpring, const FVector& InDamping, const FVector& InForceLimit)
{
	ProfileInstance.LinearDrive.SetDriveParams(InSpring, InDamping, InForceLimit);

	FPhysicsInterface::ExecuteOnUnbrokenConstraintReadWrite(ConstraintHandle, [this](const FPhysicsConstraintHandle& InUnbrokenConstraint)
	{
		FPhysicsInterface::UpdateLinearDrive_AssumesLocked(InUnbrokenConstraint, ProfileInstance.LinearDrive);
	});
}

void FConstraintInstance::SetLinearDriveAccelerationMode(const bool bAccelerationMode)
{
	ProfileInstance.LinearDrive.SetAccelerationMode(bAccelerationMode);
	FPhysicsInterface::ExecuteOnUnbrokenConstraintReadWrite(ConstraintHandle, [this](const FPhysicsConstraintHandle& InUnbrokenConstraint)
	{
		FPhysicsInterface::UpdateLinearDrive_AssumesLocked(InUnbrokenConstraint, ProfileInstance.LinearDrive);
	});
}

/** Get the linear drive's strength parameters */
void FConstraintInstance::GetLinearDriveParams(float& OutPositionStrength, float& OutVelocityStrength, float& OutForceLimit)
{
	ProfileInstance.LinearDrive.GetDriveParams(OutPositionStrength, OutVelocityStrength, OutForceLimit);
}

/** Get the linear drive's strength parameters */
void FConstraintInstance::GetLinearDriveParams(FVector& OutPositionStrength, FVector& OutVelocityStrength, FVector& OutForceLimit)
{
	ProfileInstance.LinearDrive.GetDriveParams(OutPositionStrength, OutVelocityStrength, OutForceLimit);
}

/** Function for setting target angular position. */
void FConstraintInstance::SetAngularOrientationTarget(const FQuat& InOrientationTarget)
{
	FRotator OrientationTargetRot(InOrientationTarget);

	// If settings are the same, don't do anything.
	if( ProfileInstance.AngularDrive.OrientationTarget == OrientationTargetRot )
	{
		return;
	}

	ProfileInstance.AngularDrive.OrientationTarget = OrientationTargetRot;
	FPhysicsInterface::SetDriveOrientation(ConstraintHandle, InOrientationTarget);
}

void FConstraintInstance::SetDriveParams(
	const FVector& InLinearSpring, const FVector& InLinearDamping, const FVector& InForceLimit,
	const FVector& InAngularSpring, const FVector& InAngularDamping, const FVector& InTorqueLimit,
	EAngularDriveMode::Type InAngularDriveMode)
{
	ProfileInstance.LinearDrive.SetDriveParams(InLinearSpring, InLinearDamping, InForceLimit);
	ProfileInstance.LinearDrive.SetLinearPositionDrive(
		InLinearSpring.X != 0, InLinearSpring.Y != 0, InLinearSpring.Z != 0);
	ProfileInstance.LinearDrive.SetLinearVelocityDrive(
		InLinearDamping.X != 0, InLinearDamping.Y != 0, InLinearDamping.Z != 0);

	ProfileInstance.AngularDrive.SetDriveParams(InAngularSpring, InAngularDamping, InTorqueLimit);
	ProfileInstance.AngularDrive.SetAngularDriveMode(InAngularDriveMode);
	ProfileInstance.AngularDrive.SetOrientationDriveTwistAndSwing(InAngularSpring.Y != 0, InAngularSpring.X != 0);
	ProfileInstance.AngularDrive.SetOrientationDriveSLERP(InAngularSpring.Z != 0);
	ProfileInstance.AngularDrive.SetAngularVelocityDriveTwistAndSwing(InAngularDamping.Y != 0, InAngularDamping.X != 0);
	ProfileInstance.AngularDrive.SetAngularVelocityDriveSLERP(InAngularDamping.Z != 0);

	FPhysicsInterface::ExecuteOnUnbrokenConstraintReadWrite(ConstraintHandle, [this](const FPhysicsConstraintHandle& InUnbrokenConstraint)
	{
		FPhysicsInterface::UpdateLinearDrive_AssumesLocked(InUnbrokenConstraint, ProfileInstance.LinearDrive);
		FPhysicsInterface::UpdateAngularDrive_AssumesLocked(InUnbrokenConstraint, ProfileInstance.AngularDrive);
	});
}

float FConstraintInstance::GetCurrentSwing1() const
{
	return FPhysicsInterface::GetCurrentSwing1(ConstraintHandle);
}

float FConstraintInstance::GetCurrentSwing2() const
{
	return FPhysicsInterface::GetCurrentSwing2(ConstraintHandle);
}

float FConstraintInstance::GetCurrentTwist() const
{
	return FPhysicsInterface::GetCurrentTwist(ConstraintHandle);
}


/** Function for setting target angular velocity. */
void FConstraintInstance::SetAngularVelocityTarget(const FVector& InVelTarget)
{
	// If settings are the same, don't do anything.
	if( ProfileInstance.AngularDrive.AngularVelocityTarget == InVelTarget )
	{
		return;
	}

	ProfileInstance.AngularDrive.AngularVelocityTarget = InVelTarget;
	FPhysicsInterface::SetDriveAngularVelocity(ConstraintHandle, RevolutionsToRads(InVelTarget));
}

/** Function for setting angular motor parameters. */
void FConstraintInstance::SetAngularDriveParams(float InSpring, float InDamping, float InForceLimit)
{
	ProfileInstance.AngularDrive.SetDriveParams(InSpring, InDamping, InForceLimit);

	FPhysicsInterface::ExecuteOnUnbrokenConstraintReadWrite(ConstraintHandle, [&](const FPhysicsConstraintHandle& InUnbrokenConstraint)
	{
		FPhysicsInterface::UpdateAngularDrive_AssumesLocked(InUnbrokenConstraint, ProfileInstance.AngularDrive);
	});
}

void FConstraintInstance::SetAngularDriveAccelerationMode(const bool bAccelerationMode)
{
	ProfileInstance.AngularDrive.SetAccelerationMode(bAccelerationMode);
	FPhysicsInterface::ExecuteOnUnbrokenConstraintReadWrite(ConstraintHandle, [this](const FPhysicsConstraintHandle& InUnbrokenConstraint)
	{
		FPhysicsInterface::UpdateAngularDrive_AssumesLocked(InUnbrokenConstraint, ProfileInstance.AngularDrive);
	});
}

void FConstraintInstance::GetAngularDriveParams(float& OutSpring, float& OutDamping, float& OutForceLimit) const
{
	ProfileInstance.AngularDrive.GetDriveParams(OutSpring, OutDamping, OutForceLimit);
}

/** Scale Angular Limit Constraints (as defined in RB_ConstraintSetup) */
void FConstraintInstance::SetAngularDOFLimitScale(float InSwing1LimitScale, float InSwing2LimitScale, float InTwistLimitScale)
{
	FPhysicsInterface::ExecuteOnUnbrokenConstraintReadWrite(ConstraintHandle, [&](const FPhysicsConstraintHandle& InUnbrokenConstraint)
	{
		if ( ProfileInstance.ConeLimit.Swing1Motion == ACM_Limited || ProfileInstance.ConeLimit.Swing2Motion == ACM_Limited )
		{
			if (ProfileInstance.ConeLimit.Swing1Motion == ACM_Limited)
			{
				FPhysicsInterface::SetAngularMotionLimitType_AssumesLocked(InUnbrokenConstraint, ELimitAxis::Swing1, ACM_Limited);
			}

			if (ProfileInstance.ConeLimit.Swing2Motion == ACM_Limited)
			{
				FPhysicsInterface::SetAngularMotionLimitType_AssumesLocked(InUnbrokenConstraint, ELimitAxis::Swing2, ACM_Limited);
			}
		
			//The limit values need to be clamped so it will be valid in PhysX
			float ZLimitAngle = FMath::ClampAngle(ProfileInstance.ConeLimit.Swing1LimitDegrees * InSwing1LimitScale, UE_KINDA_SMALL_NUMBER, 179.9999f) * (UE_PI/180.0f);
			float YLimitAngle = FMath::ClampAngle(ProfileInstance.ConeLimit.Swing2LimitDegrees * InSwing2LimitScale, UE_KINDA_SMALL_NUMBER, 179.9999f) * (UE_PI/180.0f);
			float LimitContactDistance =  FMath::DegreesToRadians(FMath::Max(1.f, ProfileInstance.ConeLimit.ContactDistance * FMath::Min(InSwing1LimitScale, InSwing2LimitScale)));

			FPhysicsInterface::SetSwingLimit(ConstraintHandle, YLimitAngle, ZLimitAngle, LimitContactDistance);
		}

		if ( ProfileInstance.ConeLimit.Swing1Motion  == ACM_Locked )
		{
			FPhysicsInterface::SetAngularMotionLimitType_AssumesLocked(InUnbrokenConstraint, ELimitAxis::Swing1, ACM_Locked);
		}

		if ( ProfileInstance.ConeLimit.Swing2Motion  == ACM_Locked )
		{
			FPhysicsInterface::SetAngularMotionLimitType_AssumesLocked(InUnbrokenConstraint, ELimitAxis::Swing2, ACM_Locked);
		}

		if ( ProfileInstance.TwistLimit.TwistMotion == ACM_Limited )
		{
			FPhysicsInterface::SetAngularMotionLimitType_AssumesLocked(InUnbrokenConstraint, ELimitAxis::Twist, ACM_Limited);

			const float TwistLimitRad	= ProfileInstance.TwistLimit.TwistLimitDegrees * InTwistLimitScale * (UE_PI/180.0f);
			float LimitContactDistance = FMath::DegreesToRadians(FMath::Max(1.f, ProfileInstance.ConeLimit.ContactDistance * InTwistLimitScale));

			FPhysicsInterface::SetTwistLimit(ConstraintHandle, -TwistLimitRad, TwistLimitRad, LimitContactDistance);
		}
		else if ( ProfileInstance.TwistLimit.TwistMotion == ACM_Locked )
		{
			FPhysicsInterface::SetAngularMotionLimitType_AssumesLocked(InUnbrokenConstraint, ELimitAxis::Twist, ACM_Locked);
		}
	});
}

/** Allows you to dynamically change the size of the linear limit 'sphere'. */
void FConstraintInstance::SetLinearLimitSize(float NewLimitSize)
{
	//TODO: Is this supposed to be scaling the linear limit? The code just sets it directly.
	FPhysicsInterface::ExecuteOnUnbrokenConstraintReadWrite(ConstraintHandle, [&](const FPhysicsConstraintHandle& InUnbrokenConstraint)
	{
		FPhysicsInterface::SetLinearLimit(ConstraintHandle, NewLimitSize);
	});
}

bool FConstraintInstance::Serialize(FArchive& Ar)
{
	Ar.UsingCustomVersion(FFrameworkObjectVersion::GUID);
	Ar.UsingCustomVersion(FAnimPhysObjectVersion::GUID);
	return false;	//We only have this function to mark custom GUID. Still want serialize tagged properties
}

#if WITH_EDITORONLY_DATA
void FConstraintInstance::PostSerialize(const FArchive& Ar)
{
	if (Ar.IsLoading() && Ar.UEVer() < VER_UE4_FIXUP_STIFFNESS_AND_DAMPING_SCALE)
	{
		LinearLimitStiffness_DEPRECATED		/= CVarConstraintAngularStiffnessScale.GetValueOnAnyThread();
		SwingLimitStiffness_DEPRECATED		/= CVarConstraintAngularStiffnessScale.GetValueOnAnyThread();
		TwistLimitStiffness_DEPRECATED		/= CVarConstraintAngularStiffnessScale.GetValueOnAnyThread();
		LinearLimitDamping_DEPRECATED		/=  CVarConstraintAngularDampingScale.GetValueOnAnyThread();
		SwingLimitDamping_DEPRECATED		/=  CVarConstraintAngularDampingScale.GetValueOnAnyThread();
		TwistLimitDamping_DEPRECATED		/=  CVarConstraintAngularDampingScale.GetValueOnAnyThread();
	}

	if (Ar.IsLoading() && Ar.UEVer() < VER_UE4_FIXUP_MOTOR_UNITS)
	{
		AngularVelocityTarget_DEPRECATED *= 1.f / (2.f * UE_PI);	//we want to use revolutions per second - old system was using radians directly
	}

	if (Ar.IsLoading() && Ar.UEVer() < VER_UE4_CONSTRAINT_INSTANCE_MOTOR_FLAGS)
	{
		bLinearXVelocityDrive_DEPRECATED = LinearVelocityTarget_DEPRECATED.X != 0.f;
		bLinearYVelocityDrive_DEPRECATED = LinearVelocityTarget_DEPRECATED.Y != 0.f;
		bLinearZVelocityDrive_DEPRECATED = LinearVelocityTarget_DEPRECATED.Z != 0.f;
	}
	
	if (Ar.IsLoading() && Ar.CustomVer(FFrameworkObjectVersion::GUID) < FFrameworkObjectVersion::ConstraintInstanceBehaviorParameters)
	{
		//Need to move all the deprecated properties into the new profile struct
		ProfileInstance.bDisableCollision = bDisableCollision_DEPRECATED;
		ProfileInstance.bEnableProjection = bEnableProjection_DEPRECATED;
		ProfileInstance.ProjectionLinearTolerance = ProjectionLinearTolerance_DEPRECATED;
		ProfileInstance.ProjectionAngularTolerance = ProjectionAngularTolerance_DEPRECATED;
		ProfileInstance.LinearLimit.XMotion = LinearXMotion_DEPRECATED;
		ProfileInstance.LinearLimit.YMotion = LinearYMotion_DEPRECATED;
		ProfileInstance.LinearLimit.ZMotion = LinearZMotion_DEPRECATED;
		ProfileInstance.LinearLimit.Limit = LinearLimitSize_DEPRECATED;
		ProfileInstance.LinearLimit.bSoftConstraint = bLinearLimitSoft_DEPRECATED;
		ProfileInstance.LinearLimit.Stiffness = LinearLimitStiffness_DEPRECATED;
		ProfileInstance.LinearLimit.Damping = LinearLimitDamping_DEPRECATED;
		ProfileInstance.bLinearBreakable = bLinearBreakable_DEPRECATED;
		ProfileInstance.LinearBreakThreshold = LinearBreakThreshold_DEPRECATED;
		ProfileInstance.ConeLimit.Swing1Motion = AngularSwing1Motion_DEPRECATED;
		ProfileInstance.TwistLimit.TwistMotion = AngularTwistMotion_DEPRECATED;
		ProfileInstance.ConeLimit.Swing2Motion = AngularSwing2Motion_DEPRECATED;
		ProfileInstance.ConeLimit.bSoftConstraint = bSwingLimitSoft_DEPRECATED;
		ProfileInstance.TwistLimit.bSoftConstraint = bTwistLimitSoft_DEPRECATED;
		ProfileInstance.ConeLimit.Swing1LimitDegrees = Swing1LimitAngle_DEPRECATED;
		ProfileInstance.TwistLimit.TwistLimitDegrees = TwistLimitAngle_DEPRECATED;
		ProfileInstance.ConeLimit.Swing2LimitDegrees = Swing2LimitAngle_DEPRECATED;
		ProfileInstance.ConeLimit.Stiffness = SwingLimitStiffness_DEPRECATED;
		ProfileInstance.ConeLimit.Damping = SwingLimitDamping_DEPRECATED;
		ProfileInstance.TwistLimit.Stiffness = TwistLimitStiffness_DEPRECATED;
		ProfileInstance.TwistLimit.Damping = TwistLimitDamping_DEPRECATED;
		ProfileInstance.bAngularBreakable = bAngularBreakable_DEPRECATED;
		ProfileInstance.AngularBreakThreshold = AngularBreakThreshold_DEPRECATED;

		//we no longer have a single control for all linear axes. If it was off we ensure all individual drives are off. If it's on we just leave things alone.
		//This loses a bit of info, but the ability to toggle drives on and off at runtime was very obfuscated so hopefuly this doesn't hurt too many people. They can still toggle individual drives on and off
		ProfileInstance.LinearDrive.XDrive.bEnablePositionDrive = bLinearXPositionDrive_DEPRECATED && bLinearPositionDrive_DEPRECATED;
		ProfileInstance.LinearDrive.XDrive.bEnableVelocityDrive = bLinearXVelocityDrive_DEPRECATED && bLinearVelocityDrive_DEPRECATED;
		ProfileInstance.LinearDrive.YDrive.bEnablePositionDrive = bLinearYPositionDrive_DEPRECATED && bLinearPositionDrive_DEPRECATED;
		ProfileInstance.LinearDrive.YDrive.bEnableVelocityDrive = bLinearYVelocityDrive_DEPRECATED && bLinearVelocityDrive_DEPRECATED;
		ProfileInstance.LinearDrive.ZDrive.bEnablePositionDrive = bLinearZPositionDrive_DEPRECATED && bLinearPositionDrive_DEPRECATED;
		ProfileInstance.LinearDrive.ZDrive.bEnableVelocityDrive = bLinearZVelocityDrive_DEPRECATED && bLinearVelocityDrive_DEPRECATED;
		
		ProfileInstance.LinearDrive.PositionTarget = LinearPositionTarget_DEPRECATED;
		ProfileInstance.LinearDrive.VelocityTarget = LinearVelocityTarget_DEPRECATED;

		//Linear drives now set settings per axis so duplicate old data
		ProfileInstance.LinearDrive.XDrive.Stiffness = LinearDriveSpring_DEPRECATED;
		ProfileInstance.LinearDrive.YDrive.Stiffness = LinearDriveSpring_DEPRECATED;
		ProfileInstance.LinearDrive.ZDrive.Stiffness = LinearDriveSpring_DEPRECATED;
		ProfileInstance.LinearDrive.XDrive.Damping = LinearDriveDamping_DEPRECATED;
		ProfileInstance.LinearDrive.YDrive.Damping = LinearDriveDamping_DEPRECATED;
		ProfileInstance.LinearDrive.ZDrive.Damping = LinearDriveDamping_DEPRECATED;
		ProfileInstance.LinearDrive.XDrive.MaxForce = LinearDriveForceLimit_DEPRECATED;
		ProfileInstance.LinearDrive.YDrive.MaxForce = LinearDriveForceLimit_DEPRECATED;
		ProfileInstance.LinearDrive.ZDrive.MaxForce = LinearDriveForceLimit_DEPRECATED;

		//We now expose twist swing and slerp drive directly. In the old system you had a single switch, but then there was also special switches for disabling twist and swing
		//Technically someone COULD disable these, but they are not exposed in editor so it seems very unlikely. So if they are true and angular orientation is false we override it
		ProfileInstance.AngularDrive.SwingDrive.bEnablePositionDrive = bEnableSwingDrive_DEPRECATED && bAngularOrientationDrive_DEPRECATED;
		ProfileInstance.AngularDrive.SwingDrive.bEnableVelocityDrive = bEnableSwingDrive_DEPRECATED && bAngularVelocityDrive_DEPRECATED;
		ProfileInstance.AngularDrive.TwistDrive.bEnablePositionDrive = bEnableTwistDrive_DEPRECATED && bAngularOrientationDrive_DEPRECATED;
		ProfileInstance.AngularDrive.TwistDrive.bEnableVelocityDrive = bEnableTwistDrive_DEPRECATED && bAngularVelocityDrive_DEPRECATED;
		ProfileInstance.AngularDrive.SlerpDrive.bEnablePositionDrive = bAngularOrientationDrive_DEPRECATED;
		ProfileInstance.AngularDrive.SlerpDrive.bEnableVelocityDrive = bAngularVelocityDrive_DEPRECATED;

		ProfileInstance.AngularDrive.AngularDriveMode = AngularDriveMode_DEPRECATED;
		ProfileInstance.AngularDrive.OrientationTarget = AngularOrientationTarget_DEPRECATED;
		ProfileInstance.AngularDrive.AngularVelocityTarget = AngularVelocityTarget_DEPRECATED;

		//duplicate drive spring data into all 3 drives
		ProfileInstance.AngularDrive.SwingDrive.Stiffness = AngularDriveSpring_DEPRECATED;
		ProfileInstance.AngularDrive.TwistDrive.Stiffness = AngularDriveSpring_DEPRECATED;
		ProfileInstance.AngularDrive.SlerpDrive.Stiffness = AngularDriveSpring_DEPRECATED;
		ProfileInstance.AngularDrive.SwingDrive.Damping = AngularDriveDamping_DEPRECATED;
		ProfileInstance.AngularDrive.TwistDrive.Damping = AngularDriveDamping_DEPRECATED;
		ProfileInstance.AngularDrive.SlerpDrive.Damping = AngularDriveDamping_DEPRECATED;
		ProfileInstance.AngularDrive.SwingDrive.MaxForce = AngularDriveForceLimit_DEPRECATED;
		ProfileInstance.AngularDrive.TwistDrive.MaxForce = AngularDriveForceLimit_DEPRECATED;
		ProfileInstance.AngularDrive.SlerpDrive.MaxForce = AngularDriveForceLimit_DEPRECATED;

	}

	if (Ar.IsLoading() && Ar.CustomVer(FAnimPhysObjectVersion::GUID) < FAnimPhysObjectVersion::TuneSoftLimitStiffnessAndDamping)
	{
		//Handle the fact that 0,0 used to mean hard limit, but now means free
		if(ProfileInstance.LinearLimit.Stiffness == 0.f && ProfileInstance.LinearLimit.Damping == 0.f)
		{
			ProfileInstance.LinearLimit.bSoftConstraint = false;
		}

		if (ProfileInstance.ConeLimit.Stiffness == 0.f && ProfileInstance.ConeLimit.Damping == 0.f)
		{
			ProfileInstance.ConeLimit.bSoftConstraint = false;
		}

		if (ProfileInstance.TwistLimit.Stiffness == 0.f && ProfileInstance.TwistLimit.Damping == 0.f)
		{
			ProfileInstance.TwistLimit.bSoftConstraint = false;
		}

		//Now handle the new linear spring stiffness and damping coefficient
		if(CVarConstraintAngularStiffnessScale.GetValueOnAnyThread() > 0.f)
		{
			ProfileInstance.LinearLimit.Stiffness *= CVarConstraintAngularStiffnessScale.GetValueOnAnyThread() / CVarConstraintLinearStiffnessScale.GetValueOnAnyThread();
		}

		if (CVarConstraintAngularDampingScale.GetValueOnAnyThread() > 0.f)
		{
			ProfileInstance.LinearLimit.Damping *= CVarConstraintAngularDampingScale.GetValueOnAnyThread() / CVarConstraintLinearDampingScale.GetValueOnAnyThread();
		}
	}
}
#endif

//Hacks to easily get zeroed memory for special case when we don't use GC
void FConstraintInstance::Free(FConstraintInstance * Ptr)
{
	Ptr->~FConstraintInstance();
	FMemory::Free(Ptr);
}
FConstraintInstance * FConstraintInstance::Alloc()
{
	void* Memory = FMemory::Malloc(sizeof(FConstraintInstance));
	FMemory::Memzero(Memory, sizeof(FConstraintInstance));
	return new (Memory)FConstraintInstance();
}

void FConstraintInstance::GetProjectionParams(float& ProjectionLinearAlpha, float& ProjectionAngularAlpha, float& ProjectionLinearTolerance, float& ProjectionAngularTolerance) const
{
	ProjectionLinearAlpha = ProfileInstance.ProjectionLinearAlpha;
	ProjectionAngularAlpha = ProfileInstance.ProjectionAngularAlpha;
	ProjectionLinearTolerance = ProfileInstance.ProjectionLinearTolerance;
	ProjectionAngularTolerance = ProfileInstance.ProjectionAngularTolerance;
}

void FConstraintInstance::SetProjectionParams(bool bEnableProjection, float ProjectionLinearAlpha, float ProjectionAngularAlpha, float ProjectionLinearTolerance, float ProjectionAngularTolerance)
{
	ProfileInstance.bEnableProjection = bEnableProjection;
	ProfileInstance.ProjectionLinearAlpha = ProjectionLinearAlpha;
	ProfileInstance.ProjectionAngularAlpha = ProjectionAngularAlpha;
	ProfileInstance.ProjectionLinearTolerance = ProjectionLinearTolerance;
	ProfileInstance.ProjectionAngularTolerance = ProjectionAngularTolerance;

	FPhysicsCommand::ExecuteWrite(ConstraintHandle, [&](const FPhysicsConstraintHandle& Constraint)
		{
			const float LinearAlpha = bEnableProjection ? ProjectionLinearAlpha : 0.0f;
			const float AngularAlpha = bEnableProjection ? ProjectionAngularAlpha : 0.0f;
			const float TeleportDistance = bEnableProjection ? ProjectionLinearTolerance : -1.0f;
			const float TeleportAngle = bEnableProjection ? ProjectionAngularTolerance : -1.0f;
			FPhysicsInterface::SetProjectionEnabled_AssumesLocked(Constraint, bEnableProjection, LinearAlpha, AngularAlpha, TeleportDistance, TeleportAngle);
		});
}

float FConstraintInstance::GetShockPropagationAlpha() const
{
	return ProfileInstance.ShockPropagationAlpha;
}

void FConstraintInstance::SetShockPropagationParams(bool bEnableShockPropagation, float ShockPropagationAlpha)
{
	ProfileInstance.bEnableShockPropagation = bEnableShockPropagation;
	ProfileInstance.ShockPropagationAlpha = ShockPropagationAlpha;
	FPhysicsCommand::ExecuteWrite(ConstraintHandle, [&](const FPhysicsConstraintHandle& Constraint)
		{
			FPhysicsInterface::SetShockPropagationEnabled_AssumesLocked(Constraint, bEnableShockPropagation, ShockPropagationAlpha);
		});
}

void FConstraintInstance::EnableProjection()
{
	SetProjectionParams(true, ProfileInstance.ProjectionLinearAlpha, ProfileInstance.ProjectionAngularAlpha, ProfileInstance.ProjectionLinearTolerance, ProfileInstance.ProjectionAngularTolerance);
}

void FConstraintInstance::DisableProjection()
{
	SetProjectionParams(false, ProfileInstance.ProjectionLinearAlpha, ProfileInstance.ProjectionAngularAlpha, ProfileInstance.ProjectionLinearTolerance, ProfileInstance.ProjectionAngularTolerance);
}

void FConstraintInstance::EnableParentDominates()
{
	SetParentDominates(true);
}

void FConstraintInstance::DisableParentDominates()
{
	SetParentDominates(false);
}

void FConstraintInstance::SetParentDominates(bool bParentDominates)
{
	ProfileInstance.bParentDominates = bParentDominates;

	FPhysicsCommand::ExecuteWrite(ConstraintHandle, [this, bParentDominates](const FPhysicsConstraintHandle& Constraint)
		{
			FPhysicsInterface::SetParentDominates_AssumesLocked(Constraint, bParentDominates);
		});
}

void FConstraintInstance::EnableMassConditioning()
{
	ProfileInstance.bEnableMassConditioning = true;
	FPhysicsCommand::ExecuteWrite(ConstraintHandle, [&](const FPhysicsConstraintHandle& Constraint)
	{
		FPhysicsInterface::SetMassConditioningEnabled_AssumesLocked(Constraint, true);
	});
}

void FConstraintInstance::DisableMassConditioning()
{
	ProfileInstance.bEnableMassConditioning = false;
	FPhysicsCommand::ExecuteWrite(ConstraintHandle, [&](const FPhysicsConstraintHandle& Constraint)
	{
		FPhysicsInterface::SetMassConditioningEnabled_AssumesLocked(Constraint, false);
	});
}

FConstraintInstance* FConstraintInstanceAccessor::Get() const
{
	if (Owner.IsValid())
	{
		if (USkeletalMeshComponent* SkeletalMeshComponent = Cast<USkeletalMeshComponent>(Owner.Get()))
		{
			return SkeletalMeshComponent->GetConstraintInstanceByIndex(Index);
		}
		if (UPhysicsConstraintComponent* PhysicsConstraintComponent = Cast<UPhysicsConstraintComponent>(Owner.Get()))
		{
			return &(PhysicsConstraintComponent->ConstraintInstance);
		}
#if WITH_EDITOR
		if (UPhysicsAsset* PhysicsAsset = Cast<UPhysicsAsset>(Owner.Get()))
		{
			return PhysicsAsset->GetConstraintInstanceByIndex(Index);
		}
#endif
	}
	return nullptr;
}

void FConstraintInstanceAccessor::Modify()
{
	if (Owner.IsValid())
	{
		Owner->Modify();
	}
}

// Decomposes InRot into swing and twist about the X axis (the twist-axis convention used by Chaos
// joint constraints). Only the X and W components of InRot contribute to the twist; swing is the
// remainder. Both outputs are normalized. When InRot has no meaningful X component the twist is
// the identity and swing equals InRot.
static void DecomposeSwingTwist(const FQuat& InRot, FQuat& OutSwing, FQuat& OutTwist)
{
	if (FMath::Square(InRot.X) > 1e-12)
	{
		OutTwist = FQuat(InRot.X, 0.0, 0.0, InRot.W);
		OutTwist.Normalize();
	}
	else
	{
		OutTwist = FQuat::Identity;
	}
	OutSwing = InRot * OutTwist.Inverse();
	OutSwing.Normalize();
}

// Clamps RotSwing in-place to (Swing1Limit, Swing2Limit) using axis-projection linear
// approximation: EffSwing1 = SwingAngle * axis.Z, EffSwing2 = SwingAngle * axis.Y. Each axis is
// clamped independently to match Chaos per-axis limit enforcement. Returns true if clamped.
static bool ClampSwingLinear(FQuat& RotSwing, double Swing1Limit, double Swing2Limit)
{
	if (RotSwing.W >= 1.0 - 1e-6)
	{
		return false;
	}

	FVector SwingAxis;
	double  SwingAngle;
	RotSwing.ToAxisAndAngle(SwingAxis, SwingAngle);

	const double EffSwing1 = SwingAngle * SwingAxis.Z;
	const double EffSwing2 = SwingAngle * SwingAxis.Y;

	if (FMath::Abs(EffSwing1) <= Swing1Limit && FMath::Abs(EffSwing2) <= Swing2Limit)
	{
		return false;
	}

	const double S1_Clamped = FMath::Clamp(EffSwing1, -Swing1Limit, Swing1Limit);
	const double S2_Clamped = FMath::Clamp(EffSwing2, -Swing2Limit, Swing2Limit);
	const double NewAngle   = FMath::Sqrt(FMath::Square(S1_Clamped) + FMath::Square(S2_Clamped));
	RotSwing = NewAngle > 1e-6
		? FQuat(FVector(0.0, S2_Clamped / NewAngle, S1_Clamped / NewAngle), NewAngle)
		: FQuat::Identity;
	return true;
}

// Clamps RotSwing in-place to (Swing1Limit, Swing2Limit) using the exact Chaos swing-angle
// convention: Swing1 = 4*atan2(Z, 1+W), Swing2 = 4*atan2(Y, 1+W). Each axis is clamped
// independently. Reconstruction inverts the formula via a quadratic in W whose discriminant is
// always exactly 4, giving the closed form w = (1-T)/(1+T). Returns true if clamped.
static bool ClampSwingExact(FQuat& RotSwing, double Swing1Limit, double Swing2Limit)
{
	const double EffSwing1 = 4.0 * FMath::Atan2(RotSwing.Z, 1.0 + RotSwing.W);
	const double EffSwing2 = 4.0 * FMath::Atan2(RotSwing.Y, 1.0 + RotSwing.W);

	if (FMath::Abs(EffSwing1) <= Swing1Limit && FMath::Abs(EffSwing2) <= Swing2Limit)
	{
		return false;
	}

	const double S1_Clamped = FMath::Clamp(EffSwing1, -Swing1Limit, Swing1Limit);
	const double S2_Clamped = FMath::Clamp(EffSwing2, -Swing2Limit, Swing2Limit);
	const double t1    = FMath::Tan(S1_Clamped * 0.25);
	const double t2    = FMath::Tan(S2_Clamped * 0.25);
	const double T     = t1 * t1 + t2 * t2;
	const double W_New = (1.0 - T) / (1.0 + T);
	const double Scale = 1.0 + W_New;
	RotSwing = FQuat(0.0, t2 * Scale, t1 * Scale, W_New);
	RotSwing.Normalize();
	return true;
}

FQuat FConstraintInstance::ClampDriveTargetToLimits(const FQuat& TargetRelRot, bool bExactSwingClamp) const
{
	using namespace Chaos;

	const bool bTwistLimited  = GetAngularTwistMotion()  == EAngularConstraintMotion::ACM_Limited;
	const bool bSwing1Limited = GetAngularSwing1Motion() == EAngularConstraintMotion::ACM_Limited;
	const bool bSwing2Limited = GetAngularSwing2Motion() == EAngularConstraintMotion::ACM_Limited;

	if (!bTwistLimited && !bSwing1Limited && !bSwing2Limited)
	{
		return TargetRelRot;
	}

	// Transform TargetRelRot into constraint frame space.
	//
	// Rot_ChildFrame:  orientation of the constraint frame attached to the child body, expressed
	//                  in child-body-local space. Its X axis is the twist axis; Y and Z are the
	//                  swing axes. (UE calls this Frame1 internally.)
	// Rot_ParentFrame: orientation of the constraint frame attached to the parent body, expressed
	//                  in parent-body-local space. Defines the rest/zero-angle reference.
	//                  (UE calls this Frame2 internally.)
	// Rot_ChildRelParent: rotation of the child frame relative to the parent frame; the pure joint
	//                     angle with both frame offsets factored out. Limits are in this space.
	const FQuat Rot_ChildFrame  = GetRefFrame(EConstraintFrame::Frame1).GetRotation();
	const FQuat Rot_ParentFrame = GetRefFrame(EConstraintFrame::Frame2).GetRotation();
	FQuat Rot_ChildRelParent = Rot_ParentFrame.Inverse() * TargetRelRot * Rot_ChildFrame;
	Rot_ChildRelParent.Normalize();

	FQuat RotSwing_ChildRelParent, RotTwist_ChildRelParent;
	DecomposeSwingTwist(Rot_ChildRelParent, RotSwing_ChildRelParent, RotTwist_ChildRelParent);

	bool bClamped = false;

	if (bTwistLimited)
	{
		const double TwistLimitRad = FMath::DegreesToRadians((double)GetAngularTwistLimit());
		const double TwistAngle    = (double)FPBDJointUtilities::GetTwistAngle(FRotation3(RotTwist_ChildRelParent));
		if (FMath::Abs(TwistAngle) > TwistLimitRad)
		{
			RotTwist_ChildRelParent = FQuat(FVector::XAxisVector, FMath::Clamp(TwistAngle, -TwistLimitRad, TwistLimitRad));
			bClamped = true;
		}
	}

	if (bSwing1Limited || bSwing2Limited)
	{
		const double Swing1Limit = bSwing1Limited
			? FMath::DegreesToRadians((double)GetAngularSwing1Limit()) : (double)UE_BIG_NUMBER;
		const double Swing2Limit = bSwing2Limited
			? FMath::DegreesToRadians((double)GetAngularSwing2Limit()) : (double)UE_BIG_NUMBER;

		bClamped |= bExactSwingClamp
			? ClampSwingExact(RotSwing_ChildRelParent, Swing1Limit, Swing2Limit)
			: ClampSwingLinear(RotSwing_ChildRelParent, Swing1Limit, Swing2Limit);
	}

	if (!bClamped)
	{
		return TargetRelRot;
	}

	FQuat RotClamped_ChildRelParent = RotSwing_ChildRelParent * RotTwist_ChildRelParent;
	RotClamped_ChildRelParent.Normalize();
	return Rot_ParentFrame * RotClamped_ChildRelParent * Rot_ChildFrame.Inverse();
}

bool FConstraintInstance::WidenLimitsForDriveTarget(
	const FQuat&               TargetRelRot,
	const FConstraintInstance& DefaultConstraint,
	float                      WidenMarginDegrees)
{
	using namespace Chaos;

	const bool bTwistLimited  = DefaultConstraint.GetAngularTwistMotion()  == EAngularConstraintMotion::ACM_Limited;
	const bool bSwing1Limited = DefaultConstraint.GetAngularSwing1Motion() == EAngularConstraintMotion::ACM_Limited;
	const bool bSwing2Limited = DefaultConstraint.GetAngularSwing2Motion() == EAngularConstraintMotion::ACM_Limited;

	if (!bTwistLimited && !bSwing1Limited && !bSwing2Limited)
	{
		return false;
	}

	// Transform TargetRelRot into constraint frame space (same decomposition as ClampDriveTargetToLimits).
	const FQuat Rot_ChildFrame  = GetRefFrame(EConstraintFrame::Frame1).GetRotation();
	const FQuat Rot_ParentFrame = GetRefFrame(EConstraintFrame::Frame2).GetRotation();
	FQuat Rot_ChildRelParent = Rot_ParentFrame.Inverse() * TargetRelRot * Rot_ChildFrame;
	Rot_ChildRelParent.Normalize();

	FQuat RotSwing, RotTwist;
	DecomposeSwingTwist(Rot_ChildRelParent, RotSwing, RotTwist);

	if (RotSwing.W < 0.0f)
	{
		RotSwing = FQuat(-RotSwing.X, -RotSwing.Y, -RotSwing.Z, -RotSwing.W);

		// Without this, joints whose drive target is near rest pose can have their limits blown open to
		// ~360 degrees instead of being widened by a small margin, causing the joint to become unconstrained.
		// Root cause: the Chaos angle formula 4*atan2(Z, 1+W) (see FPBDJointCachedSolver::InitRotationConstraintsSimd
		// in PBDJointCachedSolverGaussSeidel.cpp) assumes W >= 0. Both q and -q represent the same
		// rotation (quaternion double-cover), so normalizing to the positive hemisphere is safe and
		// prevents 1+W from approaching 0 (which would produce spurious ~360 degree angles).
	}

	bool bWidened = false;

	if (bTwistLimited)
	{
		const float DefaultDeg    = DefaultConstraint.GetAngularTwistLimit();
		const float TwistAngleDeg =
			FMath::Abs(FMath::RadiansToDegrees((float)FPBDJointUtilities::GetTwistAngle(FRotation3(RotTwist))));
		const float NeededDeg = TwistAngleDeg > DefaultDeg
			? TwistAngleDeg + WidenMarginDegrees : DefaultDeg;
		if (NeededDeg > DefaultDeg)
		{
			bWidened = true;
		}
		if (!FMath::IsNearlyEqual(GetAngularTwistLimit(), NeededDeg, 0.01f))
		{
			SetAngularTwistLimit(EAngularConstraintMotion::ACM_Limited, NeededDeg);
		}
	}

	if (bSwing1Limited || bSwing2Limited)
	{
		// Exact Chaos convention: Swing1 = 4*atan2(Z, 1+W), Swing2 = 4*atan2(Y, 1+W).
		const float Swing1AngleDeg =
			FMath::Abs(FMath::RadiansToDegrees((float)(4.0 * FMath::Atan2((double)RotSwing.Z, 1.0 + (double)RotSwing.W))));
		const float Swing2AngleDeg =
			FMath::Abs(FMath::RadiansToDegrees((float)(4.0 * FMath::Atan2((double)RotSwing.Y, 1.0 + (double)RotSwing.W))));

		if (bSwing1Limited)
		{
			const float DefaultDeg = DefaultConstraint.GetAngularSwing1Limit();
			const float NeededDeg  = Swing1AngleDeg > DefaultDeg
				? Swing1AngleDeg + WidenMarginDegrees : DefaultDeg;
			if (NeededDeg > DefaultDeg)
			{
				bWidened = true;
			}
			if (!FMath::IsNearlyEqual(GetAngularSwing1Limit(), NeededDeg, 0.01f))
			{
				SetAngularSwing1Limit(EAngularConstraintMotion::ACM_Limited, NeededDeg);
			}
		}

		if (bSwing2Limited)
		{
			const float DefaultDeg = DefaultConstraint.GetAngularSwing2Limit();
			const float NeededDeg  = Swing2AngleDeg > DefaultDeg
				? Swing2AngleDeg + WidenMarginDegrees : DefaultDeg;
			if (NeededDeg > DefaultDeg)
			{
				bWidened = true;
			}
			if (!FMath::IsNearlyEqual(GetAngularSwing2Limit(), NeededDeg, 0.01f))
			{
				SetAngularSwing2Limit(EAngularConstraintMotion::ACM_Limited, NeededDeg);
			}
		}
	}

	return bWidened;
}

void FConstraintInstance::RestoreAngularLimitsToDefault(const FConstraintInstance& DefaultConstraint)
{
	if (DefaultConstraint.GetAngularSwing1Motion() == EAngularConstraintMotion::ACM_Limited)
	{
		const float DefaultDeg = DefaultConstraint.GetAngularSwing1Limit();
		if (!FMath::IsNearlyEqual(GetAngularSwing1Limit(), DefaultDeg, 0.01f))
		{
			SetAngularSwing1Limit(EAngularConstraintMotion::ACM_Limited, DefaultDeg);
		}
	}
	if (DefaultConstraint.GetAngularSwing2Motion() == EAngularConstraintMotion::ACM_Limited)
	{
		const float DefaultDeg = DefaultConstraint.GetAngularSwing2Limit();
		if (!FMath::IsNearlyEqual(GetAngularSwing2Limit(), DefaultDeg, 0.01f))
		{
			SetAngularSwing2Limit(EAngularConstraintMotion::ACM_Limited, DefaultDeg);
		}
	}
	if (DefaultConstraint.GetAngularTwistMotion() == EAngularConstraintMotion::ACM_Limited)
	{
		const float DefaultDeg = DefaultConstraint.GetAngularTwistLimit();
		if (!FMath::IsNearlyEqual(GetAngularTwistLimit(), DefaultDeg, 0.01f))
		{
			SetAngularTwistLimit(EAngularConstraintMotion::ACM_Limited, DefaultDeg);
		}
	}
}

#undef LOCTEXT_NAMESPACE

