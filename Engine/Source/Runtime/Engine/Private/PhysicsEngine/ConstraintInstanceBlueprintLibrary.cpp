// Copyright Epic Games, Inc. All Rights Reserved.

#include "PhysicsEngine/ConstraintInstanceBlueprintLibrary.h"
#include "PhysicsEngine/ConstraintInstance.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ConstraintInstanceBlueprintLibrary)

//---------------------------------------------------------------------------------------------------
// 	   
// CONSTRAINT BODIES
// 
//---------------------------------------------------------------------------------------------------
void UConstraintInstanceBlueprintLibrary::GetAttachedBodyNames(
	FConstraintInstanceAccessor& Accessor,
	FName& ParentBody,
	FName& ChildBody
)
{
	if (FConstraintInstance* ConstraintInstance = Accessor.Get())
	{
		ParentBody = ConstraintInstance->GetParentBoneName();
		ChildBody = ConstraintInstance->GetChildBoneName();
	}
}
//---------------------------------------------------------------------------------------------------
//
// CONSTRAINT BEHAVIOR 
//
//---------------------------------------------------------------------------------------------------

void UConstraintInstanceBlueprintLibrary::SetDisableCollision(
	FConstraintInstanceAccessor& Accessor,
	bool bDisableCollision
	)
{
	if (FConstraintInstance* ConstraintInstance = Accessor.Get())
	{
		Accessor.Modify();
		ConstraintInstance->SetDisableCollision(bDisableCollision);
	}
}

bool UConstraintInstanceBlueprintLibrary::GetDisableCollsion(FConstraintInstanceAccessor& Accessor)
{
	if (FConstraintInstance* ConstraintInstance = Accessor.Get())
	{
		return ConstraintInstance->IsCollisionDisabled();
	}
	return true;
}

void UConstraintInstanceBlueprintLibrary::SetProjectionParams(
	FConstraintInstanceAccessor& Accessor,
	bool bEnableProjection,
	float ProjectionLinearAlpha,
	float ProjectionAngularAlpha
	)
{
	// @todo: Need to add ProjectionLinearTolerance and ProjectionAngularTolerance to SetProjectionParams
	if (FConstraintInstance* ConstraintInstance = Accessor.Get())
	{
		const float ProjectionLinearTolerance = ConstraintInstance->ProfileInstance.ProjectionLinearTolerance;
		const float ProjectionAngularTolerance = ConstraintInstance->ProfileInstance.ProjectionAngularTolerance;

		Accessor.Modify();
		ConstraintInstance->SetProjectionParams(bEnableProjection, ProjectionLinearAlpha, ProjectionAngularAlpha, ProjectionLinearTolerance, ProjectionAngularTolerance);
	}
}

void UConstraintInstanceBlueprintLibrary::GetProjectionParams(
	FConstraintInstanceAccessor& Accessor,
	bool& bEnableProjection,
	float& ProjectionLinearAlpha,
	float& ProjectionAngularAlpha
	)
{
	// @todo: Need to add ProjectionLinearTolerance and ProjectionAngularTolerance to GetProjectionParams
	if (FConstraintInstance* ConstraintInstance = Accessor.Get())
	{
		bEnableProjection = ConstraintInstance->IsProjectionEnabled();
		float ProjectionLinearTolerance = 0;
		float ProjectionAngularTolerance = 0;
		ConstraintInstance->GetProjectionParams(ProjectionLinearAlpha, ProjectionAngularAlpha, ProjectionLinearTolerance, ProjectionAngularTolerance);
	}
	else
	{
		bEnableProjection = false;
		ProjectionLinearAlpha = 0.0f;
		ProjectionAngularAlpha = 0.0f;
	}
}

void UConstraintInstanceBlueprintLibrary::SetParentDominates(
	FConstraintInstanceAccessor& Accessor,
	bool bParentDominates
	)
{
	if (FConstraintInstance* ConstraintInstance = Accessor.Get())
	{
		Accessor.Modify();

		if (bParentDominates)
		{
			ConstraintInstance->EnableParentDominates();
		}
		else
		{
			ConstraintInstance->DisableParentDominates();
		}
	}
}

bool UConstraintInstanceBlueprintLibrary::GetParentDominates(FConstraintInstanceAccessor& Accessor)
{
	if (FConstraintInstance* ConstraintInstance = Accessor.Get())
	{
		return ConstraintInstance->IsParentDominatesEnabled();
	}
	return false;
}


void UConstraintInstanceBlueprintLibrary::SetMassConditioningEnabled(
	FConstraintInstanceAccessor& Accessor,
	bool bEnableMassConditioning
)
{
	if (FConstraintInstance* ConstraintInstance = Accessor.Get())
	{
		Accessor.Modify();

		if (bEnableMassConditioning)
		{
			ConstraintInstance->EnableMassConditioning();
		}
		else
		{
			ConstraintInstance->DisableMassConditioning();
		}
	}
}

bool UConstraintInstanceBlueprintLibrary::GetMassConditioningEnabled(FConstraintInstanceAccessor& Accessor)
{
	if (FConstraintInstance* ConstraintInstance = Accessor.Get())
	{
		return ConstraintInstance->IsMassConditioningEnabled();
	}
	return false;
}

//---------------------------------------------------------------------------------------------------
//
// LINEAR LIMITS
//
//---------------------------------------------------------------------------------------------------

void UConstraintInstanceBlueprintLibrary::SetLinearLimits(
	FConstraintInstanceAccessor& Accessor,
	TEnumAsByte<ELinearConstraintMotion> XMotion,
	TEnumAsByte<ELinearConstraintMotion> YMotion,
	TEnumAsByte<ELinearConstraintMotion> ZMotion,
	float Limit
	)
{
	if (FConstraintInstance* ConstraintInstance = Accessor.Get())
	{
		Accessor.Modify();
		ConstraintInstance->SetLinearLimits(XMotion, YMotion, ZMotion, Limit);
	}
}

void UConstraintInstanceBlueprintLibrary::GetLinearLimits(
	FConstraintInstanceAccessor& Accessor,
	TEnumAsByte<ELinearConstraintMotion>& XMotion,
	TEnumAsByte<ELinearConstraintMotion>& YMotion,
	TEnumAsByte<ELinearConstraintMotion>& ZMotion,
	float& Limit
)
{
	if (FConstraintInstance* ConstraintInstance = Accessor.Get())
	{
		XMotion = ConstraintInstance->GetLinearXMotion();
		YMotion = ConstraintInstance->GetLinearYMotion();
		ZMotion = ConstraintInstance->GetLinearZMotion();
		Limit = ConstraintInstance->GetLinearLimit();
	} 
	else
	{
		XMotion = ELinearConstraintMotion::LCM_Free;
		YMotion = ELinearConstraintMotion::LCM_Free;
		ZMotion = ELinearConstraintMotion::LCM_Free;
		Limit = 0.f;
	}
}

void UConstraintInstanceBlueprintLibrary::SetLinearSoftLimitParams(
	FConstraintInstanceAccessor& Accessor,
	bool bSoftLinearLimit,
	float LinearLimitStiffness,
	float LinearLimitDamping,
	float LinearLimitRestitution,
	float LinearLimitContactDistance
)
{
	if (FConstraintInstance* ConstraintInstance = Accessor.Get())
	{
		Accessor.Modify();
		ConstraintInstance->SetSoftLinearLimitParams(bSoftLinearLimit, LinearLimitStiffness, LinearLimitDamping, LinearLimitRestitution, LinearLimitContactDistance);
	}
}

void UConstraintInstanceBlueprintLibrary::GetLinearSoftLimitParams(
	FConstraintInstanceAccessor& Accessor,
	bool& bSoftLinearLimit,
	float& LinearLimitStiffness,
	float& LinearLimitDamping,
	float& LinearLimitRestitution,
	float& LinearLimitContactDistance
)
{
	if (FConstraintInstance* ConstraintInstance = Accessor.Get())
	{
		bSoftLinearLimit = ConstraintInstance->GetIsSoftLinearLimit();
		LinearLimitStiffness = ConstraintInstance->GetSoftLinearLimitStiffness();
		LinearLimitDamping = ConstraintInstance->GetSoftLinearLimitDamping();
		LinearLimitRestitution = ConstraintInstance->GetSoftLinearLimitRestitution();
		LinearLimitContactDistance = ConstraintInstance->GetSoftLinearLimitContactDistance();
	}
	else
	{
		bSoftLinearLimit = false;
		LinearLimitStiffness = 0.;
		LinearLimitDamping = 0.;
		LinearLimitRestitution = 0.;
		LinearLimitContactDistance = 0.;

	}
}

void UConstraintInstanceBlueprintLibrary::SetLinearBreakable(
	FConstraintInstanceAccessor& Accessor,
	bool bLinearBreakable,
	float LinearBreakThreshold
	)
{
	if (FConstraintInstance* ConstraintInstance = Accessor.Get())
	{
		Accessor.Modify();
		ConstraintInstance->SetLinearBreakable(bLinearBreakable, LinearBreakThreshold);
	}
}

void UConstraintInstanceBlueprintLibrary::GetLinearBreakable(
	FConstraintInstanceAccessor& Accessor,
	bool& bLinearBreakable,
	float& LinearBreakThreshold
	)
{
	if (FConstraintInstance* ConstraintInstance = Accessor.Get())
	{
		bLinearBreakable = ConstraintInstance->IsLinearBreakable();
		LinearBreakThreshold = ConstraintInstance->GetLinearBreakThreshold();
	}
	else
	{
		bLinearBreakable = false;
		LinearBreakThreshold = 0.0f;
	}
}

void UConstraintInstanceBlueprintLibrary::SetLinearPlasticity(
	FConstraintInstanceAccessor& Accessor,
	bool bLinearPlasticity,
	float LinearPlasticityThreshold,
	TEnumAsByte<EConstraintPlasticityType> PlasticityType
	)
{
	if (FConstraintInstance* ConstraintInstance = Accessor.Get())
	{
		Accessor.Modify();
		ConstraintInstance->SetLinearPlasticity(bLinearPlasticity, LinearPlasticityThreshold,PlasticityType);
	}
}

void UConstraintInstanceBlueprintLibrary::GetLinearPlasticity(
	FConstraintInstanceAccessor& Accessor,
	bool& bLinearPlasticity,
	float& LinearPlasticityThreshold,
	TEnumAsByte<EConstraintPlasticityType>& PlasticityType
	)
{
	if (FConstraintInstance* ConstraintInstance = Accessor.Get())
	{
		bLinearPlasticity = ConstraintInstance->HasLinearPlasticity();
		LinearPlasticityThreshold = ConstraintInstance->GetLinearPlasticityThreshold();
		PlasticityType = ConstraintInstance->GetLinearPlasticityType();
	}
	else
	{
		bLinearPlasticity = false;
		LinearPlasticityThreshold = 0.0f;
		PlasticityType = EConstraintPlasticityType::CCPT_Free;
	}
}

//---------------------------------------------------------------------------------------------------
//
// Contact Transfer Scale
//
//---------------------------------------------------------------------------------------------------

void UConstraintInstanceBlueprintLibrary::SetContactTransferScale(
	FConstraintInstanceAccessor& Accessor,
	float ContactTransferScale
)
{
	if (FConstraintInstance* ConstraintInstance = Accessor.Get())
	{
		Accessor.Modify();
		ConstraintInstance->SetContactTransferScale(ContactTransferScale);
	}
}

void UConstraintInstanceBlueprintLibrary::GetContactTransferScale(
	FConstraintInstanceAccessor& Accessor,
	float& ContactTransferScale
)
{
	if (FConstraintInstance* ConstraintInstance = Accessor.Get())
	{
		ContactTransferScale = ConstraintInstance->GetContactTransferScale();
	}
	else
	{
		ContactTransferScale = 0.f;
	}
}

//---------------------------------------------------------------------------------------------------
//
// ANGULAR LIMITS 
//
//---------------------------------------------------------------------------------------------------

void UConstraintInstanceBlueprintLibrary::SetAngularLimits(
	FConstraintInstanceAccessor& Accessor,
	TEnumAsByte<EAngularConstraintMotion> Swing1MotionType,
	float Swing1LimitAngle,
	TEnumAsByte<EAngularConstraintMotion> Swing2MotionType,
	float Swing2LimitAngle,
	TEnumAsByte<EAngularConstraintMotion> TwistMotionType,
	float TwistLimitAngle
	)
{
	if (FConstraintInstance* ConstraintInstance = Accessor.Get())
	{
		Accessor.Modify();
		ConstraintInstance->SetAngularSwing1Limit(Swing1MotionType, Swing1LimitAngle);
		ConstraintInstance->SetAngularSwing2Limit(Swing2MotionType, Swing2LimitAngle);
		ConstraintInstance->SetAngularTwistLimit(TwistMotionType, TwistLimitAngle);
	}
}

void UConstraintInstanceBlueprintLibrary::GetAngularLimits(
	FConstraintInstanceAccessor& Accessor,
	TEnumAsByte<EAngularConstraintMotion>& Swing1MotionType,
	float& Swing1LimitAngle,
	TEnumAsByte<EAngularConstraintMotion>& Swing2MotionType,
	float& Swing2LimitAngle,
	TEnumAsByte<EAngularConstraintMotion>& TwistMotionType,
	float& TwistLimitAngle
	)
{
	if (FConstraintInstance* ConstraintInstance = Accessor.Get())
	{
		Swing1MotionType = ConstraintInstance->GetAngularSwing1Motion();
		Swing1LimitAngle = ConstraintInstance->GetAngularSwing1Limit();
		Swing2MotionType = ConstraintInstance->GetAngularSwing2Motion();
		Swing2LimitAngle = ConstraintInstance->GetAngularSwing2Limit();
		TwistMotionType = ConstraintInstance->GetAngularTwistMotion();
		TwistLimitAngle = ConstraintInstance->GetAngularTwistLimit();
	} 
	else
	{
		Swing1MotionType = EAngularConstraintMotion::ACM_Free;
		Swing1LimitAngle = 0.0f;
		Swing2MotionType = EAngularConstraintMotion::ACM_Free;
		Swing2LimitAngle = 0.0f;
		TwistMotionType = EAngularConstraintMotion::ACM_Free;
		TwistLimitAngle = 0.0f;
	}
}

void UConstraintInstanceBlueprintLibrary::SetAngularSoftSwingLimitParams(
	FConstraintInstanceAccessor& Accessor,
	bool bSoftSwingLimit,
	float SwingLimitStiffness,
	float SwingLimitDamping,
	float SwingLimitRestitution,
	float SwingLimitContactDistance
)
{
	if (FConstraintInstance* ConstraintInstance = Accessor.Get())
	{
		Accessor.Modify();
		ConstraintInstance->SetSoftSwingLimitParams(bSoftSwingLimit, SwingLimitStiffness, SwingLimitDamping, SwingLimitRestitution, SwingLimitContactDistance);
	}
}

void UConstraintInstanceBlueprintLibrary::GetAngularSoftSwingLimitParams(
	FConstraintInstanceAccessor& Accessor,
	bool& bSoftSwingLimit,
	float& SwingLimitStiffness,
	float& SwingLimitDamping,
	float& SwingLimitRestitution,
	float& SwingLimitContactDistance
)
{
	if (FConstraintInstance* ConstraintInstance = Accessor.Get())
	{
		bSoftSwingLimit = ConstraintInstance->GetIsSoftSwingLimit();
		SwingLimitStiffness = ConstraintInstance->GetSoftSwingLimitStiffness();
		SwingLimitDamping = ConstraintInstance->GetSoftSwingLimitDamping();
		SwingLimitRestitution = ConstraintInstance->GetSoftSwingLimitRestitution();
		SwingLimitContactDistance = ConstraintInstance->GetSoftSwingLimitContactDistance();
	}
	else
	{
		bSoftSwingLimit = false;
		SwingLimitStiffness = 0.;
		SwingLimitDamping = 0.;
		SwingLimitRestitution = 0.;
		SwingLimitContactDistance = 0.;
		
	}
}

void UConstraintInstanceBlueprintLibrary::SetAngularSoftTwistLimitParams(
	FConstraintInstanceAccessor& Accessor,
	bool bSoftTwistLimit,
	float TwistLimitStiffness,
	float TwistLimitDamping,
	float TwistLimitRestitution,
	float TwistLimitContactDistance
)
{
	if (FConstraintInstance* ConstraintInstance = Accessor.Get())
	{
		Accessor.Modify();
		ConstraintInstance->SetSoftTwistLimitParams(bSoftTwistLimit, TwistLimitStiffness, TwistLimitDamping, TwistLimitRestitution, TwistLimitContactDistance);
	}
}

void UConstraintInstanceBlueprintLibrary::GetAngularSoftTwistLimitParams(
	FConstraintInstanceAccessor& Accessor,
	bool& bSoftTwistLimit,
	float& TwistLimitStiffness,
	float& TwistLimitDamping,
	float& TwistLimitRestitution,
	float& TwistLimitContactDistance
)
{
	if (FConstraintInstance* ConstraintInstance = Accessor.Get())
	{
		bSoftTwistLimit = ConstraintInstance->GetIsSoftTwistLimit();
		TwistLimitStiffness = ConstraintInstance->GetSoftTwistLimitStiffness();
		TwistLimitDamping = ConstraintInstance->GetSoftTwistLimitDamping();
		TwistLimitRestitution = ConstraintInstance->GetSoftTwistLimitRestitution();
		TwistLimitContactDistance = ConstraintInstance->GetSoftTwistLimitContactDistance();
	}
	else
	{
		bSoftTwistLimit = false;
		TwistLimitStiffness = 0.;
		TwistLimitDamping = 0.;
		TwistLimitRestitution = 0.;
		TwistLimitContactDistance = 0.;

	}
}

void UConstraintInstanceBlueprintLibrary::SetAngularBreakable(
	FConstraintInstanceAccessor& Accessor,
	bool bAngularBreakable,
	float AngularBreakThreshold
	)
{
	if (FConstraintInstance* ConstraintInstance = Accessor.Get())
	{
		Accessor.Modify();
		ConstraintInstance->SetAngularBreakable(bAngularBreakable, AngularBreakThreshold);
	}
}

void UConstraintInstanceBlueprintLibrary::GetAngularBreakable(
	FConstraintInstanceAccessor& Accessor,
	bool& bAngularBreakable,
	float& AngularBreakThreshold
	)
{
	if (FConstraintInstance* ConstraintInstance = Accessor.Get())
	{
		bAngularBreakable = ConstraintInstance->IsAngularBreakable();
		AngularBreakThreshold = ConstraintInstance->GetAngularBreakThreshold();
	}
	else
	{
		bAngularBreakable = false;
		AngularBreakThreshold = 0.0f;
	}

}

void UConstraintInstanceBlueprintLibrary::SetAngularPlasticity(
	FConstraintInstanceAccessor& Accessor,
	bool bAngularPlasticity,
	float AngularPlasticityThreshold
	)
{
	if (FConstraintInstance* ConstraintInstance = Accessor.Get())
	{
		Accessor.Modify();
		ConstraintInstance->SetAngularPlasticity(bAngularPlasticity, AngularPlasticityThreshold);
	}
}

void UConstraintInstanceBlueprintLibrary::GetAngularPlasticity(
	FConstraintInstanceAccessor& Accessor,
	bool& bAngularPlasticity,
	float& AngularPlasticityThreshold
	)
{
	if (FConstraintInstance* ConstraintInstance = Accessor.Get())
	{
		bAngularPlasticity = ConstraintInstance->HasAngularPlasticity();
		AngularPlasticityThreshold = ConstraintInstance->GetAngularPlasticityThreshold();
	}
	else
	{
		bAngularPlasticity = false;
		AngularPlasticityThreshold = 0.0f;
	}
}


//---------------------------------------------------------------------------------------------------
//
// LINEAR MOTOR
//
//---------------------------------------------------------------------------------------------------

void UConstraintInstanceBlueprintLibrary::SetLinearPositionDrive(
	FConstraintInstanceAccessor& Accessor, 
	bool bEnableDriveX, 
	bool bEnableDriveY, 
	bool bEnableDriveZ
	)
{
	if (FConstraintInstance* ConstraintInstance = Accessor.Get())
	{
		Accessor.Modify();
		ConstraintInstance->SetLinearPositionDrive(bEnableDriveX, bEnableDriveY, bEnableDriveZ);
	}
}

void UConstraintInstanceBlueprintLibrary::GetLinearPositionDrive(
	FConstraintInstanceAccessor& Accessor,
	bool& bEnableDriveX,
	bool& bEnableDriveY,
	bool& bEnableDriveZ
	)
{
	if (FConstraintInstance* ConstraintInstance = Accessor.Get())
	{
		bEnableDriveX = ConstraintInstance->IsLinearPositionDriveXEnabled();
		bEnableDriveY = ConstraintInstance->IsLinearPositionDriveYEnabled();
		bEnableDriveZ = ConstraintInstance->IsLinearPositionDriveZEnabled();
	}
	else
	{
		bEnableDriveX = false;
		bEnableDriveY = false;
		bEnableDriveZ = false;
	}
}

void UConstraintInstanceBlueprintLibrary::SetLinearVelocityDrive(
	FConstraintInstanceAccessor& Accessor,
	bool bEnableDriveX, 
	bool bEnableDriveY, 
	bool bEnableDriveZ
	)
{
	if (FConstraintInstance* ConstraintInstance = Accessor.Get())
	{
		Accessor.Modify();
		ConstraintInstance->SetLinearVelocityDrive(bEnableDriveX, bEnableDriveY, bEnableDriveZ);
	}
}

void UConstraintInstanceBlueprintLibrary::GetLinearVelocityDrive(
	FConstraintInstanceAccessor& Accessor,
	bool& bEnableDriveX,
	bool& bEnableDriveY,
	bool& bEnableDriveZ
)
{
	if (FConstraintInstance* ConstraintInstance = Accessor.Get())
	{
		bEnableDriveX = ConstraintInstance->IsLinearVelocityDriveXEnabled();
		bEnableDriveY = ConstraintInstance->IsLinearVelocityDriveYEnabled();
		bEnableDriveZ = ConstraintInstance->IsLinearVelocityDriveZEnabled();
	}
	else
	{
		bEnableDriveX = false;
		bEnableDriveY = false;
		bEnableDriveZ = false;
	}
}

void UConstraintInstanceBlueprintLibrary::SetLinearPositionTarget(
	FConstraintInstanceAccessor& Accessor, 
	const FVector& InPosTarget
	)
{
	if (FConstraintInstance* ConstraintInstance = Accessor.Get())
	{
		Accessor.Modify();
		ConstraintInstance->SetLinearPositionTarget(InPosTarget);
	}
}

void UConstraintInstanceBlueprintLibrary::GetLinearPositionTarget(
	FConstraintInstanceAccessor& Accessor,
	FVector& OutPosTarget
	)
{
	if (FConstraintInstance* ConstraintInstance = Accessor.Get())
	{
		OutPosTarget = ConstraintInstance->GetLinearPositionTarget();
	}
	else
	{
		OutPosTarget = FVector::ZeroVector;
	}
}


void UConstraintInstanceBlueprintLibrary::SetLinearVelocityTarget(
	FConstraintInstanceAccessor& Accessor, 
	const FVector& InVelTarget
	)
{
	if (FConstraintInstance* ConstraintInstance = Accessor.Get())
	{
		Accessor.Modify();
		ConstraintInstance->SetLinearVelocityTarget(InVelTarget);
	}
}

void UConstraintInstanceBlueprintLibrary::GetLinearVelocityTarget(
	FConstraintInstanceAccessor& Accessor,
	FVector& OutVelTarget
	)
{
	if (FConstraintInstance* ConstraintInstance = Accessor.Get())
	{
		OutVelTarget = ConstraintInstance->GetLinearVelocityTarget();
	}
	else
	{
		OutVelTarget = FVector::ZeroVector;
	}
}

void UConstraintInstanceBlueprintLibrary::SetLinearDriveParams(
	FConstraintInstanceAccessor& Accessor,
	float PositionStrength, 
	float VelocityStrength, 
	float InForceLimit
	)
{
	if (FConstraintInstance* ConstraintInstance = Accessor.Get())
	{
		Accessor.Modify();
		ConstraintInstance->SetLinearDriveParams(PositionStrength, VelocityStrength, InForceLimit);
	}
}

void UConstraintInstanceBlueprintLibrary::GetLinearDriveParams(
	FConstraintInstanceAccessor& Accessor,
	float& OutPositionStrength,
	float& OutVelocityStrength,
	float& OutForceLimit
	)
{
	if (FConstraintInstance* ConstraintInstance = Accessor.Get())
	{
		ConstraintInstance->GetLinearDriveParams(OutPositionStrength, OutVelocityStrength, OutForceLimit);
	}
	else
	{
		OutPositionStrength = 0.0f;
		OutVelocityStrength = 0.0f;
		OutForceLimit = 0.0f;
	}
}


//---------------------------------------------------------------------------------------------------
//
// ANGULAR MOTOR
//
//---------------------------------------------------------------------------------------------------

void UConstraintInstanceBlueprintLibrary::SetOrientationDriveTwistAndSwing(
	FConstraintInstanceAccessor& Accessor, 
	bool bEnableTwistDrive, 
	bool bEnableSwingDrive
	)
{
	if (FConstraintInstance* ConstraintInstance = Accessor.Get())
	{
		Accessor.Modify();
		ConstraintInstance->SetOrientationDriveTwistAndSwing(bEnableTwistDrive, bEnableSwingDrive);
	}
}

void UConstraintInstanceBlueprintLibrary::GetOrientationDriveTwistAndSwing(
	UPARAM(ref) FConstraintInstanceAccessor& Accessor,
	bool& bOutEnableTwistDrive,
	bool& bOutEnableSwingDrive
	)
{
	if (FConstraintInstance* ConstraintInstance = Accessor.Get())
	{
		ConstraintInstance->GetOrientationDriveTwistAndSwing(bOutEnableTwistDrive, bOutEnableSwingDrive);
	}
	else
	{
		bOutEnableTwistDrive = false;
		bOutEnableSwingDrive = false;
	}
}

void UConstraintInstanceBlueprintLibrary::SetOrientationDriveSLERP(
	FConstraintInstanceAccessor& Accessor, 
	bool bEnableSLERP
	)
{
	if (FConstraintInstance* ConstraintInstance = Accessor.Get())
	{
		Accessor.Modify();
		ConstraintInstance->SetOrientationDriveSLERP(bEnableSLERP);
	}
}

void UConstraintInstanceBlueprintLibrary::GetOrientationDriveSLERP(
	FConstraintInstanceAccessor& Accessor,
	bool& bOutEnableSLERP
	)
{
	if (FConstraintInstance* ConstraintInstance = Accessor.Get())
	{
		bOutEnableSLERP = ConstraintInstance->GetOrientationDriveSLERP();
	}
	else
	{
		bOutEnableSLERP = false;
	}
}

void UConstraintInstanceBlueprintLibrary::SetAngularVelocityDriveTwistAndSwing(
	FConstraintInstanceAccessor& Accessor, 
	bool bEnableTwistDrive, 
	bool bEnableSwingDrive
	)
{
	if (FConstraintInstance* ConstraintInstance = Accessor.Get())
	{
		Accessor.Modify();
		ConstraintInstance->SetAngularVelocityDriveTwistAndSwing(bEnableTwistDrive, bEnableSwingDrive);
	}
}

void UConstraintInstanceBlueprintLibrary::GetAngularVelocityDriveTwistAndSwing(
	FConstraintInstanceAccessor& Accessor,
	bool& bOutEnableTwistDrive,
	bool& bOutEnableSwingDrive
)
{
	if (FConstraintInstance* ConstraintInstance = Accessor.Get())
	{
		ConstraintInstance->GetAngularVelocityDriveTwistAndSwing(bOutEnableTwistDrive, bOutEnableSwingDrive);
	}
	else
	{
		bOutEnableTwistDrive = false;
		bOutEnableSwingDrive = false;
	}
}

void UConstraintInstanceBlueprintLibrary::SetAngularVelocityDriveSLERP(
	FConstraintInstanceAccessor& Accessor,
	bool bEnableSLERP
	)
{
	if (FConstraintInstance* ConstraintInstance = Accessor.Get())
	{
		Accessor.Modify();
		ConstraintInstance->SetAngularVelocityDriveSLERP(bEnableSLERP);
	}
}

void UConstraintInstanceBlueprintLibrary::GetAngularVelocityDriveSLERP(
	FConstraintInstanceAccessor& Accessor,
	bool& bOutEnableSLERP
	)
{
	if (FConstraintInstance* ConstraintInstance = Accessor.Get())
	{
		bOutEnableSLERP = ConstraintInstance->GetAngularVelocityDriveSLERP();
	} 
	else
	{
		bOutEnableSLERP = false;
	}
}

void UConstraintInstanceBlueprintLibrary::SetAngularDriveMode(
	FConstraintInstanceAccessor& Accessor,
	EAngularDriveMode::Type DriveMode
	)
{
	if (FConstraintInstance* ConstraintInstance = Accessor.Get())
	{
		Accessor.Modify();
		ConstraintInstance->SetAngularDriveMode(DriveMode);
	}
}

void UConstraintInstanceBlueprintLibrary::GetAngularDriveMode(
	FConstraintInstanceAccessor& Accessor,
	TEnumAsByte<EAngularDriveMode::Type>& OutDriveMode
)
{
	if (FConstraintInstance* ConstraintInstance = Accessor.Get())
	{
		OutDriveMode = ConstraintInstance->GetAngularDriveMode();
	} 
	else
	{
		OutDriveMode = EAngularDriveMode::Type::SLERP;
	}

}

void UConstraintInstanceBlueprintLibrary::SetAngularOrientationTarget(
	FConstraintInstanceAccessor& Accessor,
	const FRotator& InPosTarget
	)
{
	if (FConstraintInstance* ConstraintInstance = Accessor.Get())
	{
		Accessor.Modify();
		ConstraintInstance->SetAngularOrientationTarget(InPosTarget.Quaternion());
	}
}

void UConstraintInstanceBlueprintLibrary::GetAngularOrientationTarget(
	FConstraintInstanceAccessor& Accessor,
	FRotator& OutPosTarget
)
{
	if (FConstraintInstance* ConstraintInstance = Accessor.Get())
	{
		OutPosTarget = ConstraintInstance->GetAngularOrientationTarget();
	}
	else
	{
		OutPosTarget = FRotator::ZeroRotator;
	}
}

void UConstraintInstanceBlueprintLibrary::SetAngularVelocityTarget(
	FConstraintInstanceAccessor& Accessor,
	const FVector& InVelTarget
	)
{
	if (FConstraintInstance* ConstraintInstance = Accessor.Get())
	{
		Accessor.Modify();
		ConstraintInstance->SetAngularVelocityTarget(InVelTarget);
	}
}

void UConstraintInstanceBlueprintLibrary::GetAngularVelocityTarget(
	FConstraintInstanceAccessor& Accessor,
	FVector& OutVelTarget
)
{
	if (FConstraintInstance* ConstraintInstance = Accessor.Get())
	{
		OutVelTarget = ConstraintInstance->GetAngularVelocityTarget();
	}
	else
	{
		OutVelTarget = FVector::ZeroVector;
	}
}

void UConstraintInstanceBlueprintLibrary::SetAngularDriveParams(
	FConstraintInstanceAccessor& Accessor,
	float PositionStrength,
	float VelocityStrength,
	float InForceLimit
	)
{
	if (FConstraintInstance* ConstraintInstance = Accessor.Get())
	{
		Accessor.Modify();
		ConstraintInstance->SetAngularDriveParams(PositionStrength, VelocityStrength, InForceLimit);
	}
}

void UConstraintInstanceBlueprintLibrary::GetAngularDriveParams(
	FConstraintInstanceAccessor& Accessor,
	float& OutPositionStrength,
	float& OutVelocityStrength,
	float& OutForceLimit
)
{
	if (FConstraintInstance* ConstraintInstance = Accessor.Get())
	{
		ConstraintInstance->GetAngularDriveParams(OutPositionStrength, OutVelocityStrength, OutForceLimit);
	}
	else
	{
		OutPositionStrength = 0.0f;
		OutVelocityStrength = 0.0f;
		OutForceLimit = 0.0f;
	}
}

void UConstraintInstanceBlueprintLibrary::CopyParams(
	FConstraintInstanceAccessor& Accessor,
	FConstraintInstanceAccessor& SourceAccessor,
	bool bKeepPosition,
	bool bKeepRotation
)
{
	FConstraintInstance* Target = Accessor.Get();
	FConstraintInstance* Source = SourceAccessor.Get();

	if(Target && Source)
	{
		Accessor.Modify();
		Target->CopyConstraintPhysicalPropertiesFrom(Source, bKeepPosition, bKeepRotation);
	}
}
