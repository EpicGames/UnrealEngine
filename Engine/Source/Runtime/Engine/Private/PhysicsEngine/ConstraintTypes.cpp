// Copyright Epic Games, Inc. All Rights Reserved.

#include "PhysicsEngine/ConstraintTypes.h"
#include "HAL/IConsoleManager.h"
#include "Physics/Experimental/PhysInterface_Chaos.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ConstraintTypes)

extern TAutoConsoleVariable<float> CVarConstraintLinearDampingScale;
extern TAutoConsoleVariable<float> CVarConstraintLinearStiffnessScale;
extern TAutoConsoleVariable<float> CVarConstraintAngularDampingScale;
extern TAutoConsoleVariable<float> CVarConstraintAngularStiffnessScale;

/** Util for setting linear movement for an axis */
void SetLinearMovement_AssumesLocked(const FPhysicsConstraintHandle& InConstraintRef, PhysicsInterfaceTypes::ELimitAxis InAxis, ELinearConstraintMotion Motion, bool bLockLimitSize, bool bSkipSoftLimit)
{
	if(Motion == LCM_Locked || (Motion == LCM_Limited && bLockLimitSize))
	{
		FPhysicsInterface::SetLinearMotionLimitType_AssumesLocked(InConstraintRef, InAxis, ELinearConstraintMotion::LCM_Locked);
	}
	else
	{
		FPhysicsInterface::SetLinearMotionLimitType_AssumesLocked(InConstraintRef, InAxis, Motion);
	}

	if(bSkipSoftLimit && Motion == LCM_Limited)
	{
		FPhysicsInterface::SetLinearMotionLimitType_AssumesLocked(InConstraintRef, InAxis, ELinearConstraintMotion::LCM_Free);
	}
}


FConstraintBaseParams::FConstraintBaseParams()
	: Stiffness(50.f)
	, Damping(5.f)
	, Restitution(0.f)
	, ContactDistance(1.f)
	, bSoftConstraint(false)
{

}

FLinearConstraint::FLinearConstraint()
	: Limit(0.f)
	, XMotion(LCM_Locked)
	, YMotion(LCM_Locked)
	, ZMotion(LCM_Locked)
{
	ContactDistance = 5.f;
	Stiffness = 0.f;
	Damping = 0.f;
}

FConeConstraint::FConeConstraint()
	: Swing1LimitDegrees(45.f)
	, Swing2LimitDegrees(45.f)
	, Swing1Motion(ACM_Free)
	, Swing2Motion(ACM_Free)
{
	bSoftConstraint = true;
	ContactDistance = 1.f;
}

FTwistConstraint::FTwistConstraint()
	: TwistLimitDegrees(45)
	, TwistMotion(ACM_Free)
{
	bSoftConstraint = true;
	ContactDistance = 1.f;
}

bool ShouldSkipSoftLimits(float Stiffness, float Damping, float AverageMass)
{
	return ((Stiffness * AverageMass) == 0.f && (Damping * AverageMass) == 0.f);
}

void FLinearConstraint::UpdateLinearLimit_AssumesLocked(const FPhysicsConstraintHandle& InConstraintRef, float AverageMass, float Scale) const
{
	const float UseLimit = FMath::Max(Limit * Scale, UE_KINDA_SMALL_NUMBER);	//physx doesn't ever want limit of 0
	const bool bLockLimitSize = (UseLimit < RB_MinSizeToLockDOF);  //(TODO):Joint should not be locked under soft constraint. 
	
	const bool bSkipSoft = bSoftConstraint && ShouldSkipSoftLimits(Stiffness, Damping, AverageMass);

	SetLinearMovement_AssumesLocked(InConstraintRef, PhysicsInterfaceTypes::ELimitAxis::X, XMotion, bLockLimitSize, bSkipSoft);
	SetLinearMovement_AssumesLocked(InConstraintRef, PhysicsInterfaceTypes::ELimitAxis::Y, YMotion, bLockLimitSize, bSkipSoft);
	SetLinearMovement_AssumesLocked(InConstraintRef, PhysicsInterfaceTypes::ELimitAxis::Z, ZMotion, bLockLimitSize, bSkipSoft);

	// If any DOF is locked/limited, set up the joint limit
	if (XMotion != LCM_Free || YMotion != LCM_Free || ZMotion != LCM_Free)
	{
		FPhysicsInterface::UpdateLinearLimitParams_AssumesLocked(InConstraintRef, UseLimit, AverageMass, *this);
	}
}

void FConeConstraint::UpdateConeLimit_AssumesLocked(const FPhysicsConstraintHandle& InConstraintRef, float AverageMass) const
{
	if (Swing1Motion == ACM_Limited || Swing2Motion == ACM_Limited)
	{
		FPhysicsInterface::UpdateConeLimitParams_AssumesLocked(InConstraintRef, AverageMass, *this);
	}

	const bool bSkipSoftLimits = bSoftConstraint && ShouldSkipSoftLimits(Stiffness, Damping, AverageMass);

	FPhysicsInterface::SetAngularMotionLimitType_AssumesLocked(InConstraintRef, PhysicsInterfaceTypes::ELimitAxis::Swing1, (bSkipSoftLimits && Swing1Motion == ACM_Limited) ? EAngularConstraintMotion::ACM_Free : Swing1Motion.GetValue());
	FPhysicsInterface::SetAngularMotionLimitType_AssumesLocked(InConstraintRef, PhysicsInterfaceTypes::ELimitAxis::Swing2, (bSkipSoftLimits && Swing2Motion == ACM_Limited) ? EAngularConstraintMotion::ACM_Free : Swing2Motion.GetValue());
}

void FTwistConstraint::UpdateTwistLimit_AssumesLocked(const FPhysicsConstraintHandle& InConstraintRef, float AverageMass) const
{
	if (TwistMotion == ACM_Limited)
	{
		FPhysicsInterface::UpdateTwistLimitParams_AssumesLocked(InConstraintRef, AverageMass, *this);
	}

	const bool bSkipSoftLimits = bSoftConstraint && ShouldSkipSoftLimits(Stiffness, Damping, AverageMass);
	FPhysicsInterface::SetAngularMotionLimitType_AssumesLocked(InConstraintRef, PhysicsInterfaceTypes::ELimitAxis::Twist, (bSkipSoftLimits && TwistMotion == ACM_Limited) ? EAngularConstraintMotion::ACM_Free : TwistMotion.GetValue());
}

