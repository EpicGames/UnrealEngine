// Copyright Epic Games, Inc. All Rights Reserved.

#include "PhysicsEngine/ConstraintUtils.h"
#include "PhysicsEngine/ConstraintInstance.h"

namespace ConstraintUtils
{
	void ConfigureAsHinge(FConstraintInstance& ConstraintInstance, bool bOverwriteLimits)
	{
		ConstraintInstance.ProfileInstance.LinearLimit.XMotion = LCM_Locked;
		ConstraintInstance.ProfileInstance.LinearLimit.YMotion = LCM_Locked;
		ConstraintInstance.ProfileInstance.LinearLimit.ZMotion = LCM_Locked;

		ConstraintInstance.ProfileInstance.ConeLimit.Swing1Motion = ACM_Locked;
		ConstraintInstance.ProfileInstance.ConeLimit.Swing2Motion = ACM_Locked;
		ConstraintInstance.ProfileInstance.TwistLimit.TwistMotion = ACM_Free;

		if(bOverwriteLimits)
		{
			ConstraintInstance.ProfileInstance.ConeLimit.Swing1LimitDegrees = 0.f;
			ConstraintInstance.ProfileInstance.ConeLimit.Swing2LimitDegrees = 0.f;
			ConstraintInstance.ProfileInstance.TwistLimit.TwistLimitDegrees = 45.f;
		}

		ConstraintInstance.UpdateLinearLimit();
		ConstraintInstance.UpdateAngularLimit();
	}


	void ConfigureAsPrismatic(FConstraintInstance& ConstraintInstance, bool bOverwriteLimits)
	{
		ConstraintInstance.ProfileInstance.LinearLimit.XMotion = LCM_Free;
		ConstraintInstance.ProfileInstance.LinearLimit.YMotion = LCM_Locked;
		ConstraintInstance.ProfileInstance.LinearLimit.ZMotion = LCM_Locked;
		ConstraintInstance.ProfileInstance.ConeLimit.Swing1Motion = ACM_Locked;
		ConstraintInstance.ProfileInstance.ConeLimit.Swing2Motion = ACM_Locked;
		ConstraintInstance.ProfileInstance.TwistLimit.TwistMotion = ACM_Locked;

		if (bOverwriteLimits)
		{
			ConstraintInstance.ProfileInstance.ConeLimit.Swing1LimitDegrees = 0.0f;
			ConstraintInstance.ProfileInstance.ConeLimit.Swing2LimitDegrees = 0.0f;
			ConstraintInstance.ProfileInstance.TwistLimit.TwistLimitDegrees = 0.0f;
		}

		ConstraintInstance.UpdateLinearLimit();
		ConstraintInstance.UpdateAngularLimit();
	}

	void ConfigureAsSkelJoint(FConstraintInstance& ConstraintInstance, bool bOverwriteLimits)
	{
		ConstraintInstance.ProfileInstance.LinearLimit.XMotion = LCM_Locked;
		ConstraintInstance.ProfileInstance.LinearLimit.YMotion = LCM_Locked;
		ConstraintInstance.ProfileInstance.LinearLimit.ZMotion = LCM_Locked;
		ConstraintInstance.ProfileInstance.ConeLimit.Swing1Motion = ACM_Limited;
		ConstraintInstance.ProfileInstance.ConeLimit.Swing2Motion = ACM_Limited;
		ConstraintInstance.ProfileInstance.TwistLimit.TwistMotion = ACM_Limited;

		if (bOverwriteLimits)
		{
			ConstraintInstance.ProfileInstance.ConeLimit.Swing1LimitDegrees = 45.0f;
			ConstraintInstance.ProfileInstance.ConeLimit.Swing2LimitDegrees = 45.0f;
			ConstraintInstance.ProfileInstance.TwistLimit.TwistLimitDegrees = 15.0f;
		}

		ConstraintInstance.UpdateLinearLimit();
		ConstraintInstance.UpdateAngularLimit();
	}

	void ConfigureAsBallAndSocket(FConstraintInstance& ConstraintInstance, bool bOverwriteLimits)
	{
		ConstraintInstance.ProfileInstance.LinearLimit.XMotion = LCM_Locked;
		ConstraintInstance.ProfileInstance.LinearLimit.YMotion = LCM_Locked;
		ConstraintInstance.ProfileInstance.LinearLimit.ZMotion = LCM_Locked;
		ConstraintInstance.ProfileInstance.ConeLimit.Swing1Motion = ACM_Free;
		ConstraintInstance.ProfileInstance.ConeLimit.Swing2Motion = ACM_Free;
		ConstraintInstance.ProfileInstance.TwistLimit.TwistMotion = ACM_Free;

		if (bOverwriteLimits)
		{
			ConstraintInstance.ProfileInstance.ConeLimit.Swing1LimitDegrees = 0.0f;
			ConstraintInstance.ProfileInstance.ConeLimit.Swing2LimitDegrees = 0.0f;
			ConstraintInstance.ProfileInstance.TwistLimit.TwistLimitDegrees = 0.0f;
		}

		ConstraintInstance.UpdateLinearLimit();
		ConstraintInstance.UpdateAngularLimit();
	}

	bool IsHinge(const FConstraintInstance& ConstraintInstance)
	{
		int32 AngularDOF = 0;
		AngularDOF += ConstraintInstance.GetAngularSwing1Motion() != ACM_Locked ? 1 : 0;
		AngularDOF += ConstraintInstance.GetAngularSwing2Motion() != ACM_Locked ? 1 : 0;
		AngularDOF += ConstraintInstance.GetAngularTwistMotion() != ACM_Locked ? 1 : 0;

		return	ConstraintInstance.GetLinearXMotion() == LCM_Locked &&
			ConstraintInstance.GetLinearYMotion() == LCM_Locked &&
			ConstraintInstance.GetLinearZMotion() == LCM_Locked &&
			AngularDOF == 1;
	}

	bool IsPrismatic(const FConstraintInstance& ConstraintInstance)
	{
		int32 LinearDOF = 0;
		LinearDOF += ConstraintInstance.GetLinearXMotion() != LCM_Locked ? 1 : 0;
		LinearDOF += ConstraintInstance.GetLinearYMotion() != LCM_Locked ? 1 : 0;
		LinearDOF += ConstraintInstance.GetLinearZMotion() != LCM_Locked ? 1 : 0;

		return	LinearDOF > 0 &&
			ConstraintInstance.GetAngularSwing1Motion() == ACM_Locked &&
			ConstraintInstance.GetAngularSwing2Motion() == ACM_Locked &&
			ConstraintInstance.GetAngularTwistMotion() == ACM_Locked;
	}

	bool IsSkelJoint(const FConstraintInstance& ConstraintInstance)
	{
		return	ConstraintInstance.GetLinearXMotion() == LCM_Locked &&
			ConstraintInstance.GetLinearYMotion() == LCM_Locked &&
			ConstraintInstance.GetLinearZMotion() == LCM_Locked &&
			ConstraintInstance.GetAngularSwing1Motion() == ACM_Limited &&
			ConstraintInstance.GetAngularSwing2Motion() == ACM_Limited &&
			ConstraintInstance.GetAngularTwistMotion() == ACM_Limited;
	}

	bool IsBallAndSocket(const FConstraintInstance& ConstraintInstance)
	{
		int32 AngularDOF = 0;
		AngularDOF += ConstraintInstance.GetAngularSwing1Motion() != ACM_Locked ? 1 : 0;
		AngularDOF += ConstraintInstance.GetAngularSwing2Motion() != ACM_Locked ? 1 : 0;
		AngularDOF += ConstraintInstance.GetAngularTwistMotion() != ACM_Locked ? 1 : 0;

		return	ConstraintInstance.GetLinearXMotion() == LCM_Locked &&
			ConstraintInstance.GetLinearYMotion() == LCM_Locked &&
			ConstraintInstance.GetLinearZMotion() == LCM_Locked &&
			AngularDOF > 1;
	}

}
