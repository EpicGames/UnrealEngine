// Copyright Epic Games, Inc. All Rights Reserved.

#include "Physics/DefaultPhysicsQueryHandler.h"

#include "Physics/GenericPhysicsInterface.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DefaultPhysicsQueryHandler)

bool UDefaultPhysicsQueryHandler::Overlap(const EQueryInfo InfoType, const EThreadQueryContext ThreadContext, const UWorld* World, const FOverlapQueryData& OverlapData, const FCommonQueryData& CommonData, TArray<FOverlapResult>& OutOverlaps)
{
	return Chaos::Private::FQueryInterface_Internal::Overlap(InfoType, ThreadContext, World, OverlapData, CommonData, OutOverlaps);
}

bool UDefaultPhysicsQueryHandler::RaycastTest(const EThreadQueryContext ThreadContext, const UWorld* World, const FRayQueryData& RayData, const FCommonQueryData& CommonData, FHitResult& OutHits)
{
	return Chaos::Private::FQueryInterface_Internal::RaycastTest(ThreadContext, World, RayData, CommonData, OutHits);
}

bool UDefaultPhysicsQueryHandler::RaycastSingle(const EThreadQueryContext ThreadContext, const UWorld* World, const FRayQueryData& RayData, const FCommonQueryData& CommonData, FHitResult& OutHits)
{
	return Chaos::Private::FQueryInterface_Internal::RaycastSingle(ThreadContext, World, RayData, CommonData, OutHits);
}

bool UDefaultPhysicsQueryHandler::RaycastMulti(const EThreadQueryContext ThreadContext, const UWorld* World, const FRayQueryData& RayData, const FCommonQueryData& CommonData, TArray<FHitResult>& OutHits)
{
	return Chaos::Private::FQueryInterface_Internal::RaycastMulti(ThreadContext, World, RayData, CommonData, OutHits);
}

bool UDefaultPhysicsQueryHandler::SweepTest(const EThreadQueryContext ThreadContext, const UWorld* World, const FSweepQueryData& SweepData, const FCommonQueryData& CommonData, FHitResult& OutHits)
{
	return Chaos::Private::FQueryInterface_Internal::SweepTest(ThreadContext, World, SweepData, CommonData, OutHits);
}

bool UDefaultPhysicsQueryHandler::SweepSingle(const EThreadQueryContext ThreadContext, const UWorld* World, const FSweepQueryData& SweepData, const FCommonQueryData& CommonData, FHitResult& OutHits)
{
	return Chaos::Private::FQueryInterface_Internal::SweepSingle(ThreadContext, World, SweepData, CommonData, OutHits);
}

bool UDefaultPhysicsQueryHandler::SweepMulti(const EThreadQueryContext ThreadContext, const UWorld* World, const FSweepQueryData& SweepData, const FCommonQueryData& CommonData, TArray<FHitResult>& OutHits)
{
	return Chaos::Private::FQueryInterface_Internal::SweepMulti(ThreadContext, World, SweepData, CommonData, OutHits);
}
