// Copyright Epic Games, Inc. All Rights Reserved.

#include "PhysicsEngine/EnvironmentalCollisions.h"
#include "Chaos/Capsule.h"
#include "Chaos/Convex.h"
#include "Chaos/ImplicitFwd.h"
#include "Chaos/ImplicitObjectScaled.h"
#include "Chaos/PhysicsObjectInterface.h"
#include "Physics/Experimental/PhysScene_Chaos.h"
#include "Components/PrimitiveComponent.h"
#include "Components/SceneComponent.h"
#include "Components/SkeletalMeshComponent.h"
#include "Engine/OverlapResult.h"
#include "Engine/World.h"
#include "Misc/ConfigCacheIni.h"
#include "ClothCollisionData.h"
#include "ClothingSimulationInstance.h"

void FEnvironmentalCollisions::AppendCollisionDataFromEnvironment(const USceneComponent* SceneComponent, FClothCollisionData& CollisionData)
{

	// Read config on first call
	struct FEnvironmentCollisionConfig
	{
		float Padding = 2.f;  // Extra padding added to the bounds so that the collision can still be detected after being resolved
		float Thickness = 2.f;  // Extra thickness added to edgy collision shapes (cubes & convexes)
		int32 MaxShapes = 32;  // Limit the number of extracted shapes per component as these collisions are very expensive

		FEnvironmentCollisionConfig()
		{
			if (GConfig)
			{
				GConfig->GetFloat(TEXT("ClothSettings"), TEXT("EnvironmentCollisionPadding"), Padding, GEngineIni);
				GConfig->GetFloat(TEXT("ClothSettings"), TEXT("EnvironmentCollisionThickness"), Thickness, GEngineIni);
				GConfig->GetInt(TEXT("ClothSettings"), TEXT("EnvironmentCollisionMaxShapes"), MaxShapes, GEngineIni);
			}
		}
	};
	static FEnvironmentCollisionConfig EnvironmentCollisionConfig;

	TArray<FOverlapResult> Overlaps;

	FCollisionObjectQueryParams ObjectParams;

	ObjectParams.AddObjectTypesToQuery(ECollisionChannel::ECC_WorldStatic);

	FCollisionQueryParams Params(SCENE_QUERY_STAT(ClothOverlapComponents), false);
	const FBoxSphereBounds PaddedBounds = SceneComponent->Bounds.ExpandBy(EnvironmentCollisionConfig.Padding);

	SceneComponent->GetWorld()->OverlapMultiByObjectType(Overlaps, PaddedBounds.Origin, FQuat::Identity, ObjectParams, FCollisionShape::MakeBox(PaddedBounds.BoxExtent), Params);

	for (int32 OverlapIdx = 0; OverlapIdx < Overlaps.Num(); ++OverlapIdx)
	{
		const TWeakObjectPtr<UPrimitiveComponent>& Component = Overlaps[OverlapIdx].Component;
		const Chaos::FPhysicsObjectHandle PhysicsObject = Overlaps[OverlapIdx].PhysicsObject;

		const UPrimitiveComponent* const ResolvedComponent = Component.Get();

		// Resolve the FBodyInstance for this overlap.
		// When Component is valid with a valid body instance (the common case), the existing component-based path is used.
		// When Component is null (e.g. FastGeo ISM without surrogate) or the Component's body instance is invalid (ISM with surrogate), fall back to the
		// FBodyInstance obtained directly from the physics proxy.
		const bool bUseResolvedComponent = ResolvedComponent && ResolvedComponent->BodyInstance.IsValidBodyInstance();

		FBodyInstance* FallbackBodyInstance = nullptr;
		if (!bUseResolvedComponent && PhysicsObject)
		{
			const Chaos::FConstPhysicsObjectHandle ConstHandle = PhysicsObject;
			IPhysicsProxyBase* Proxy = Chaos::FPhysicsObjectInterface::GetProxy({ &ConstHandle, 1 });
			if (Proxy)
			{
				FallbackBodyInstance = SceneComponent->GetWorld()->GetPhysicsScene()->GetBodyInstanceFromProxy(Proxy);
			}
		}

		const FBodyInstance* const BodyInstToUse = bUseResolvedComponent ? &ResolvedComponent->BodyInstance : FallbackBodyInstance;

		if (!BodyInstToUse || !BodyInstToUse->IsValidBodyInstance())
		{
			continue;
		}

		const ECollisionChannel Channel = bUseResolvedComponent ? ResolvedComponent->GetCollisionObjectType() : FallbackBodyInstance->GetObjectType();

		if (Channel == ECollisionChannel::ECC_WorldStatic)
		{
			// Static world geo
			bool bSuccessfulRead = false;

			FPhysicsActorHandle ActorRef = BodyInstToUse->GetPhysicsActorWeldRoot();

			const FString OwnerDebugName = ResolvedComponent && ResolvedComponent->GetOwner()
				? ResolvedComponent->GetOwner()->GetFName().ToString()
				: TEXT("Unknown (ISM instance)");

			FTransform ClothComponentTransform = SceneComponent->GetComponentTransform();
			ClothComponentTransform.RemoveScaling();  // The environment collision shape doesn't need the scale of the cloth skeletal mesh applied to it (but it does need the source scale from its component transform)

			FPhysicsCommand::ExecuteRead(ActorRef, [&](const FPhysicsActorHandle& Actor)
				{
					// Get the transform directly from the PhysicsActor when not using the resolved component
					const FTransform ComponentToClothBaseTransform = bUseResolvedComponent ? ResolvedComponent->GetComponentTransform() * ClothComponentTransform.Inverse() : FPhysicsInterface::GetTransform_AssumesLocked(Actor) * ClothComponentTransform.Inverse();

					TArray<FPhysicsShapeHandle> AllShapes;
					const int32 NumSyncShapes = BodyInstToUse->GetAllShapes_AssumesLocked(AllShapes);

					if (NumSyncShapes == 0 || NumSyncShapes > EnvironmentCollisionConfig.MaxShapes)
					{
						// Either no shapes or too complicated to consider
						return;
					}

					using namespace Chaos;

					const FReal Thickness = (FReal)EnvironmentCollisionConfig.Thickness;

					auto AddSphere = [&CollisionData](const FImplicitSphere3& ImplicitSphere, const FTransform& ComponentToClothTransform, const FVec3& Scale = FVec3::OneVector)
						{
							CollisionData.Spheres.Emplace(
								ImplicitSphere.GetRadiusf() * Scale.X,  // Assumes uniform scale
								ComponentToClothTransform.TransformPosition(FVec3(ImplicitSphere.GetCenterf())));
						};

					auto AddBox = [&CollisionData, Thickness](const FImplicitBox3& ImplicitBox, const FTransform& ComponentToClothTransform, const FVec3& Scale = FVec3::OneVector)
						{
							CollisionData.Boxes.Emplace(
								ComponentToClothTransform.TransformPosition(ImplicitBox.GetCenter()),
								ComponentToClothTransform.GetRotation(),
								ImplicitBox.Extents() * Scale * (FReal)0.5 + Thickness);
						};

					auto AddCapsule = [&CollisionData](const FImplicitCapsule3& ImplicitCapsule, const FTransform& ComponentToClothTransform, const FVec3& Scale = FVec3::OneVector)
						{
							const int32 BaseSphereIndex = CollisionData.Spheres.Num();

							const FReal Radius = ImplicitCapsule.GetRadiusf() * Scale.X;  // Assumes uniform scale
							const FReal HalfHeight = ImplicitCapsule.GetHeightf() * (FReal)0.5;
							const FVector TransformedAxis = ComponentToClothTransform.TransformVectorNoScale(FVec3(ImplicitCapsule.GetAxisf()));
							const FVector HalfSegment = TransformedAxis * HalfHeight * Scale.X;  // Assumes uniform scale
							const FVector TransformedCenter = ComponentToClothTransform.TransformPosition(FVec3(ImplicitCapsule.GetCenterf()));

							CollisionData.Spheres.Emplace(Radius, TransformedCenter + HalfSegment);
							CollisionData.Spheres.Emplace(Radius, TransformedCenter - HalfSegment);

							CollisionData.SphereConnections.Emplace(BaseSphereIndex, BaseSphereIndex + 1);
						};

					auto AddConvex = [&CollisionData, Thickness](const FImplicitConvex3& ImplicitConvex, const FMatrix& ComponentToClothMatrix, const FVec3* const Scale = nullptr)
						{
							TArray<FClothCollisionPrim_ConvexFace> Faces;
							const int32 NumPlanes = ImplicitConvex.NumPlanes();
							Faces.SetNum(NumPlanes);

							TArray<FVector> SurfacePoints;
							const int32 NumSurfacePoints = ImplicitConvex.NumVertices();
							SurfacePoints.SetNumUninitialized(NumSurfacePoints);

							TArray<TArray<int32, TInlineAllocator<4>>, TInlineAllocator<16>> PointFaces;
							PointFaces.SetNum(NumSurfacePoints);

							for (int32 FaceIndex = 0; FaceIndex < NumPlanes; ++FaceIndex)
							{
								const TPlaneConcrete<FReal, 3>& Plane = ImplicitConvex.GetPlane(FaceIndex);
								const FVec3& Normal = Plane.Normal();
								const FVec3 Base = Plane.X() + Normal * (FReal)Thickness;

								Faces[FaceIndex].Plane = FPlane(FVector(Base), FVector(Normal)).TransformBy(ComponentToClothMatrix);

								const int32 NumFaceIndices = ImplicitConvex.NumPlaneVertices(FaceIndex);
								Faces[FaceIndex].Indices.SetNumUninitialized(NumFaceIndices);

								for (int32 Index = 0; Index < NumFaceIndices; ++Index)
								{
									const int32 PointIndex = ImplicitConvex.GetPlaneVertex(FaceIndex, Index);
									Faces[FaceIndex].Indices[Index] = PointIndex;

									if (ensure(PointFaces.IsValidIndex(PointIndex)))
									{
										PointFaces[PointIndex].Add(FaceIndex);
									}
								}
							}

							for (int32 PointIndex = 0; PointIndex < NumSurfacePoints; ++PointIndex)
							{
								if (ensure(PointFaces[PointIndex].Num() >= 3))
								{
									const int32 Index0 = PointFaces[PointIndex][0];
									const int32 Index1 = PointFaces[PointIndex][1];
									const int32 Index2 = PointFaces[PointIndex][2];

									if (!FMath::IntersectPlanes3(SurfacePoints[PointIndex], Faces[Index0].Plane, Faces[Index1].Plane, Faces[Index2].Plane))
									{
										SurfacePoints[PointIndex] = ComponentToClothMatrix.TransformPosition(FVector(ImplicitConvex.GetVertex(PointIndex)));
									}
								}
								else
								{
									SurfacePoints[PointIndex] = ComponentToClothMatrix.TransformPosition(FVector(ImplicitConvex.GetVertex(PointIndex)));
								}
							}

							CollisionData.Convexes.Emplace(MoveTemp(Faces), MoveTemp(SurfacePoints));
						};

					bool bHasSimpleCollision = false;

					for (FPhysicsShapeHandle& ShapeHandle : AllShapes)
					{
						FTransform ComponentToClothTransform = ComponentToClothBaseTransform;

						const FImplicitObject* ImplicitObject = &ShapeHandle.GetGeometry();
						EImplicitObjectType ImplicitType = ImplicitObject->GetType();

						// Transformed implicits
						if (ImplicitType == ImplicitObjectType::Transformed)
						{
							const TImplicitObjectTransformed<FReal, 3>& ImplicitTransformed = ImplicitObject->GetObjectChecked<TImplicitObjectTransformed<FReal, 3>>();
							ImplicitObject = ImplicitTransformed.GetTransformedObject();
							ImplicitType = ImplicitObject->GetType();

							ComponentToClothTransform = ImplicitTransformed.GetTransform() * ComponentToClothTransform;
							UE_LOGF(LogSkeletalMesh, Verbose, "Found transformed environmental collision");
						}

						switch (ImplicitType)
						{
							// Base implicits
						case ImplicitObjectType::Sphere:
						{
							const FImplicitSphere3& ImplicitSphere = ImplicitObject->GetObjectChecked<FImplicitSphere3>();;
							AddSphere(ImplicitSphere, ComponentToClothTransform);
						}
						UE_LOGF(LogSkeletalMesh, Verbose, "Found Sphere cloth environmental collision in [%ls]", *OwnerDebugName);
						bHasSimpleCollision = true;
						break;
						case ImplicitObjectType::Box:
						{
							const FImplicitBox3& ImplicitBox = ImplicitObject->GetObjectChecked<FImplicitBox3>();
							AddBox(ImplicitBox, ComponentToClothTransform);
						}
						UE_LOGF(LogSkeletalMesh, Verbose, "Found Box cloth environmental collision in [%ls]", *OwnerDebugName);
						bHasSimpleCollision = true;
						break;
						case ImplicitObjectType::Capsule:
						{
							const FImplicitCapsule3& ImplicitCapsule = ImplicitObject->GetObjectChecked<FImplicitCapsule3>();
							AddCapsule(ImplicitCapsule, ComponentToClothTransform);
							UE_LOGF(LogSkeletalMesh, Verbose, "Found Capsule cloth environmental collision in [%ls]", *OwnerDebugName);
							bHasSimpleCollision = true;
						}
						break;
						case ImplicitObjectType::Convex:
						{
							const FImplicitConvex3& ImplicitConvex = ImplicitObject->GetObjectChecked<FImplicitConvex3>();
							AddConvex(ImplicitConvex, ComponentToClothTransform.ToMatrixNoScale());
							UE_LOGF(LogSkeletalMesh, Verbose, "Found Convex cloth environmental collision in [%ls]", *OwnerDebugName);
							bHasSimpleCollision = true;
						}
						break;

						// Instanced implicits
						case ImplicitObjectType::IsInstanced | ImplicitObjectType::Sphere:
						{
							const TImplicitObjectInstanced<FImplicitSphere3>& ImplicitInstanced = ImplicitObject->GetObjectChecked<TImplicitObjectInstanced<FImplicitSphere3>>();
							check(ImplicitInstanced.Object());
							const FImplicitSphere3& ImplicitSphere = *ImplicitInstanced.GetInstancedObject();
							AddSphere(ImplicitSphere, ComponentToClothTransform);
							UE_LOGF(LogSkeletalMesh, Verbose, "Found Instanced Sphere cloth environmental collision in [%ls]", *OwnerDebugName);
							bHasSimpleCollision = true;
						}
						break;
						case ImplicitObjectType::IsInstanced | ImplicitObjectType::Box:
						{
							const TImplicitObjectInstanced<FImplicitBox3>& ImplicitInstanced = ImplicitObject->GetObjectChecked<TImplicitObjectInstanced<FImplicitBox3>>();
							check(ImplicitInstanced.Object());
							const FImplicitBox3& ImplicitBox = *ImplicitInstanced.GetInstancedObject();
							AddBox(ImplicitBox, ComponentToClothTransform);
							UE_LOGF(LogSkeletalMesh, Verbose, "Found Instanced Box cloth environmental collision in [%ls]", *OwnerDebugName);
							bHasSimpleCollision = true;
						}
						break;
						case ImplicitObjectType::IsInstanced | ImplicitObjectType::Capsule:
						{
							const TImplicitObjectInstanced<FImplicitCapsule3>& ImplicitInstanced = ImplicitObject->GetObjectChecked<TImplicitObjectInstanced<FImplicitCapsule3>>();
							check(ImplicitInstanced.Object());
							const FImplicitCapsule3& ImplicitCapsule = *ImplicitInstanced.GetInstancedObject();
							AddCapsule(ImplicitCapsule, ComponentToClothTransform);
							UE_LOGF(LogSkeletalMesh, Verbose, "Found Instanced Capsule cloth environmental collision in [%ls]", *OwnerDebugName);
							bHasSimpleCollision = true;
						}
						break;
						case ImplicitObjectType::IsInstanced | ImplicitObjectType::Convex:
						{
							const TImplicitObjectInstanced<FImplicitConvex3>& ImplicitInstanced = ImplicitObject->GetObjectChecked<TImplicitObjectInstanced<FImplicitConvex3>>();
							check(ImplicitInstanced.Object());
							const FImplicitConvex3& ImplicitConvex = *ImplicitInstanced.GetInstancedObject();
							AddConvex(ImplicitConvex, ComponentToClothTransform.ToMatrixNoScale());
							UE_LOGF(LogSkeletalMesh, Verbose, "Found Instanced Convex cloth environmental collision in [%ls]", *OwnerDebugName);
							bHasSimpleCollision = true;
						}
						break;

						// Scaled implicits
						case ImplicitObjectType::IsScaled | ImplicitObjectType::Sphere:
						{
							const TImplicitObjectScaled<FImplicitSphere3>& ImplicitScaled = ImplicitObject->GetObjectChecked<TImplicitObjectScaled<FImplicitSphere3>>();
							check(ImplicitScaled.Object());
							const FImplicitSphere3& ImplicitSphere = *ImplicitScaled.GetUnscaledObject();
							ensure(FVector::DistSquared(ComponentToClothTransform.GetScale3D(), FVector(ImplicitScaled.GetScale())) < UE_KINDA_SMALL_NUMBER);
							AddSphere(ImplicitSphere, ComponentToClothTransform, ImplicitScaled.GetScale());
							UE_LOGF(LogSkeletalMesh, Verbose, "Found Scaled Sphere cloth environmental collision in [%ls]", *OwnerDebugName);
							bHasSimpleCollision = true;
						}
						break;
						case ImplicitObjectType::IsScaled | ImplicitObjectType::Box:
						{
							const TImplicitObjectScaled<FImplicitBox3>& ImplicitScaled = ImplicitObject->GetObjectChecked<TImplicitObjectScaled<FImplicitBox3>>();
							check(ImplicitScaled.Object());
							const FImplicitBox3& ImplicitBox = *ImplicitScaled.GetUnscaledObject();
							ensure(FVector::DistSquared(ComponentToClothTransform.GetScale3D(), FVector(ImplicitScaled.GetScale())) < UE_KINDA_SMALL_NUMBER);
							AddBox(ImplicitBox, ComponentToClothTransform, ImplicitScaled.GetScale());
							UE_LOGF(LogSkeletalMesh, Verbose, "Found Scaled Box cloth environmental collision in [%ls]", *OwnerDebugName);
							bHasSimpleCollision = true;
						}
						break;
						case ImplicitObjectType::IsScaled | ImplicitObjectType::Capsule:
						{
							const TImplicitObjectScaled<FImplicitCapsule3>& ImplicitScaled = ImplicitObject->GetObjectChecked<TImplicitObjectScaled<FImplicitCapsule3>>();
							check(ImplicitScaled.Object());
							const FImplicitCapsule3& ImplicitCapsule = *ImplicitScaled.GetUnscaledObject();
							ensure(FVector::DistSquared(ComponentToClothTransform.GetScale3D(), FVector(ImplicitScaled.GetScale())) < UE_KINDA_SMALL_NUMBER);
							AddCapsule(ImplicitCapsule, ComponentToClothTransform, ImplicitScaled.GetScale());
							UE_LOGF(LogSkeletalMesh, Verbose, "Found Scaled Capsule cloth environmental collision in [%ls]", *OwnerDebugName);
							bHasSimpleCollision = true;
						}
						break;
						case ImplicitObjectType::IsScaled | ImplicitObjectType::Convex:
						{
							const TImplicitObjectScaled<FImplicitConvex3>& ImplicitScaled = ImplicitObject->GetObjectChecked<TImplicitObjectScaled<FImplicitConvex3>>();
							check(ImplicitScaled.Object());
							const FImplicitConvex3& ImplicitConvex = *ImplicitScaled.GetUnscaledObject();
							ensure(FVector::DistSquared(ComponentToClothTransform.GetScale3D(), FVector(ImplicitScaled.GetScale())) < UE_KINDA_SMALL_NUMBER);
							AddConvex(ImplicitConvex, ComponentToClothTransform.ToMatrixWithScale());
							UE_LOGF(LogSkeletalMesh, Verbose, "Found Scaled Convex cloth environmental collision in [%ls]", *OwnerDebugName);
							bHasSimpleCollision = true;
						}
						break;

						// Triangle mesh
						case ImplicitObjectType::TriangleMesh:
						case ImplicitObjectType::IsInstanced | ImplicitObjectType::TriangleMesh:
						case ImplicitObjectType::IsScaled | ImplicitObjectType::TriangleMesh:
							// TODO: We could eventually want to collide cloth against triangle meshes,
							//       however the concept of simple vs complex shape might need to be clarified
							//       as it currently iterates over all shape to discard the triangle mesh ones.
							UE_LOGF(LogSkeletalMesh, Verbose, "Found unusable Triangle Mesh cloth environmental collision in [%ls]", *OwnerDebugName);
							break;

						default:
							UE_LOGF(LogSkeletalMesh, Verbose, "Found unsupported collision type during environmental collision with the cloth in [%ls]", *OwnerDebugName);
							break;
						}
					}
					bSuccessfulRead = true;
				});
		}
	}
}
