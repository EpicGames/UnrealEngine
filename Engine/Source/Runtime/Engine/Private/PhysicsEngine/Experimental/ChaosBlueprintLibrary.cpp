// Copyright Epic Games, Inc. All Rights Reserved.
#include "PhysicsEngine/ChaosBlueprintLibrary.h"
#include "Engine/Engine.h"
#include "Physics/Experimental/ChaosEventRelay.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ChaosBlueprintLibrary)

const UChaosEventRelay* UChaosBlueprintLibrary::GetEventRelayFromContext(UObject* ContextObject)
{
	if (GEngine && ContextObject)
	{
		UWorld* World = GEngine->GetWorldFromContextObject(ContextObject, EGetWorldErrorMode::ReturnNull);
		if (World)
		{
			return World->GetChaosEventRelay();
		}
	}
	return nullptr;
}
