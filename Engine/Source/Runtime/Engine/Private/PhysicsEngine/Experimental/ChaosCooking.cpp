// Copyright Epic Games, Inc. All Rights Reserved.

#include "PhysicsEngine/Experimental/ChaosCooking.h"
#include "Chaos/CollisionConvexMesh.h"
#include "ChaosDerivedDataUtil.h"
#include "Chaos/Convex.h"
#include "Chaos/TriangleMeshImplicitObject.h"

int32 EnableMeshClean = 1;
FAutoConsoleVariableRef CVarEnableMeshClean(TEXT("p.EnableMeshClean"), EnableMeshClean, TEXT("Enable/Disable mesh cleanup during cook."));

namespace Chaos
{
	namespace Cooking
	{
		static void CopyUpdatedFaceRemapFromTriangleMesh(const Chaos::FTriangleMeshImplicitObject& TriangleMesh, TArray<int32>& OutFaceRemap)
		{
			for (int32 TriangleIndex = 0; TriangleIndex < OutFaceRemap.Num(); TriangleIndex++)
			{
				OutFaceRemap[TriangleIndex] = TriangleMesh.GetExternalFaceIndexFromInternal(TriangleIndex);
			}
		}

		Chaos::FTriangleMeshImplicitObjectPtr BuildSingleTrimesh(const FTriMeshCollisionData& Desc, TArray<int32>& OutFaceRemap, TArray<int32>& OutVertexRemap)
		{
			if(Desc.Vertices.Num() == 0)
			{
				return nullptr;
			}

			TArray<FVector3f> FinalVerts = Desc.Vertices;

			// Push indices into one flat array
			TArray<int32> FinalIndices;
			FinalIndices.Reserve(Desc.Indices.Num() * 3);
			for(const FTriIndices& Tri : Desc.Indices)
			{
				// NOTE: This is where the Winding order of the triangles are changed to be consistent throughout the rest of the physics engine
				// After this point we should have clockwise (CW) winding in left handed (LH) coordinates (or equivalently CCW in RH)
				// This is the opposite convention followed in most of the unreal engine
				FinalIndices.Add(Desc.bFlipNormals ? Tri.v1 : Tri.v0);
				FinalIndices.Add(Desc.bFlipNormals ? Tri.v0 : Tri.v1);
				FinalIndices.Add(Tri.v2);
			}

			if(EnableMeshClean)
			{
				Chaos::CleanTrimesh(FinalVerts, FinalIndices, &OutFaceRemap, &OutVertexRemap);
			}

			// Build particle list #BG Maybe allow TParticles to copy vectors?
			Chaos::FTriangleMeshImplicitObject::ParticlesType TriMeshParticles;
			TriMeshParticles.AddParticles(FinalVerts.Num());

			const int32 NumVerts = FinalVerts.Num();
			for(int32 VertIndex = 0; VertIndex < NumVerts; ++VertIndex)
			{
				TriMeshParticles.SetX(VertIndex, FinalVerts[VertIndex]);
			}

			// Build chaos triangle list. #BGTODO Just make the clean function take these types instead of double copying
			auto LambdaHelper = [&Desc, &FinalVerts, &FinalIndices, &TriMeshParticles, &OutFaceRemap, &OutVertexRemap](auto& Triangles) -> Chaos::FTriangleMeshImplicitObjectPtr
			{
				const int32 NumTriangles = FinalIndices.Num() / 3;
				bool bHasMaterials = Desc.MaterialIndices.Num() > 0;
				TArray<uint16> MaterialIndices;

				if(bHasMaterials)
				{
					MaterialIndices.Reserve(NumTriangles);
				}

				// Need to rebuild face remap array, in case there are any invalid triangles
				TArray<int32> OldFaceRemap = MoveTemp(OutFaceRemap);
				OutFaceRemap.Reserve(OldFaceRemap.Num());

				Triangles.Reserve(NumTriangles);
				for(int32 TriangleIndex = 0; TriangleIndex < NumTriangles; ++TriangleIndex)
				{
					// Only add this triangle if it is valid
					const int32 BaseIndex = TriangleIndex * 3;
					const bool bIsValidTriangle = Chaos::FConvexBuilder::IsValidTriangle(
						FinalVerts[FinalIndices[BaseIndex]],
						FinalVerts[FinalIndices[BaseIndex + 1]],
						FinalVerts[FinalIndices[BaseIndex + 2]]);

					// TODO: Figure out a proper way to handle this. Could these edges get sewn together? Is this important?
					//if (ensureMsgf(bIsValidTriangle, TEXT("FChaosDerivedDataCooker::BuildTriangleMeshes(): Trimesh attempted cooked with invalid triangle!")));
					if(bIsValidTriangle)
					{
						Triangles.Add(Chaos::TVector<int32, 3>(FinalIndices[BaseIndex], FinalIndices[BaseIndex + 1], FinalIndices[BaseIndex + 2]));
						if (!OldFaceRemap.IsEmpty())
						{
							OutFaceRemap.Add(OldFaceRemap[TriangleIndex]);
						}

						if(bHasMaterials)
						{
							if(EnableMeshClean)
							{
								if(!ensure(OldFaceRemap.IsValidIndex(TriangleIndex)))
								{
									MaterialIndices.Empty();
									bHasMaterials = false;
								}
								else
								{
									const int32 OriginalIndex = OldFaceRemap[TriangleIndex];

									if(ensure(Desc.MaterialIndices.IsValidIndex(OriginalIndex)))
									{
										MaterialIndices.Add(Desc.MaterialIndices[OriginalIndex]);
									}
									else
									{
										MaterialIndices.Empty();
										bHasMaterials = false;
									}
								}
							}
							else
							{
								if(ensure(Desc.MaterialIndices.IsValidIndex(TriangleIndex)))
								{
									MaterialIndices.Add(Desc.MaterialIndices[TriangleIndex]);
								}
								else
								{
									MaterialIndices.Empty();
									bHasMaterials = false;
								}
							}

						}
					}
				}

				TUniquePtr<TArray<int32>> OutFaceRemapPtr = MakeUnique<TArray<int32>>(OutFaceRemap);
				TUniquePtr<TArray<int32>> OutVertexRemapPtr = Chaos::TriMeshPerPolySupport ? MakeUnique<TArray<int32>>(OutVertexRemap) : nullptr;

				Chaos::FTriangleMeshImplicitObject::FBuildSettings MeshBuildSettings;
				MeshBuildSettings.PerFaceBounds = Desc.bReducedMemoryRepresentation ? Chaos::FTrimeshBVH::EPerFaceBoundsRepresentation::Byte : Chaos::FTrimeshBVH::EPerFaceBoundsRepresentation::VectorizedFloat;
				constexpr bool bCullsBackFaceRaycast = false;
				Chaos::FTriangleMeshImplicitObjectPtr TriangleMesh( new Chaos::FTriangleMeshImplicitObject(MoveTemp(TriMeshParticles), MoveTemp(Triangles), MoveTemp(MaterialIndices), MoveTemp(OutFaceRemapPtr), MoveTemp(OutVertexRemapPtr),
					bCullsBackFaceRaycast, MeshBuildSettings));

				// Propagate remapped indices from the FTriangleMeshImplicitObject back to the remap array
				CopyUpdatedFaceRemapFromTriangleMesh(*TriangleMesh.GetReference(), OutFaceRemap);

				return TriangleMesh;
			};

			if(FinalVerts.Num() < TNumericLimits<uint16>::Max())
			{
				TArray<Chaos::TVector<uint16, 3>> TrianglesSmallIdx;
				return LambdaHelper(TrianglesSmallIdx);
			}
			else
			{
				TArray<Chaos::TVector<int32, 3>> TrianglesLargeIdx;
				return LambdaHelper(TrianglesLargeIdx);
			}
		}

		void BuildConvexMeshes(TArray<Chaos::FImplicitObjectPtr>& OutConvexMeshes, const FCookBodySetupInfo& InParams)
		{
			using namespace Chaos;
			auto BuildConvexFromVerts = [](TArray<Chaos::FImplicitObjectPtr>& OutConvexes, const TArray<TArray<FVector>>& InMeshVerts, const bool bMirrored)
			{
				for(const TArray<FVector>& HullVerts : InMeshVerts)
				{
					const int32 NumHullVerts = HullVerts.Num();
					if(NumHullVerts == 0)
					{
						continue;
					}

					// Calculate the margin to apply to the convex - it depends on overall dimensions
					FAABB3 Bounds = FAABB3::EmptyAABB();
					for(int32 VertIndex = 0; VertIndex < NumHullVerts; ++VertIndex)
					{
						const FVector& HullVert = HullVerts[VertIndex];
						Bounds.GrowToInclude(HullVert);
					}

					// Create the corner vertices for the convex
					TArray<FConvex::FVec3Type> ConvexVertices;
					ConvexVertices.SetNumZeroed(NumHullVerts);

					for(int32 VertIndex = 0; VertIndex < NumHullVerts; ++VertIndex)
					{
						const FVector& HullVert = HullVerts[VertIndex];
						ConvexVertices[VertIndex] = FConvex::FVec3Type(bMirrored ? -HullVert.X : HullVert.X, HullVert.Y, HullVert.Z);
					}

					// Margin is always zero on convex shapes - they are intended to be instanced
					OutConvexes.Emplace(new Chaos::FConvex(ConvexVertices, 0.0f));
				}
			};

			if(InParams.bCookNonMirroredConvex)
			{
				BuildConvexFromVerts(OutConvexMeshes, InParams.NonMirroredConvexVertices, false);
			}

			if(InParams.bCookMirroredConvex)
			{
				BuildConvexFromVerts(OutConvexMeshes, InParams.MirroredConvexVertices, true);
			}
		}

		void BuildTriangleMeshes(TArray<Chaos::FTriangleMeshImplicitObjectPtr>& OutTriangleMeshes, TArray<int32>& OutFaceRemap, TArray<int32>& OutVertexRemap, const FCookBodySetupInfo& InParams)
		{
			if(!InParams.bCookTriMesh)
			{
				return;
			}

			TArray<FVector3f> FinalVerts = InParams.TriangleMeshDesc.Vertices;

			// Push indices into one flat array
			TArray<int32> FinalIndices;
			FinalIndices.Reserve(InParams.TriangleMeshDesc.Indices.Num() * 3);
			for(const FTriIndices& Tri : InParams.TriangleMeshDesc.Indices)
			{
				// NOTE: This is where the Winding order of the triangles are changed to be consistent throughout the rest of the physics engine
				// After this point we should have clockwise (CW) winding in left handed (LH) coordinates (or equivalently CCW in RH)
				// This is the opposite convention followed in most of the unreal engine
				FinalIndices.Add(InParams.TriangleMeshDesc.bFlipNormals ? Tri.v1 : Tri.v0);
				FinalIndices.Add(InParams.TriangleMeshDesc.bFlipNormals ? Tri.v0 : Tri.v1);
				FinalIndices.Add(Tri.v2);
			}

			if(EnableMeshClean)
			{
				Chaos::CleanTrimesh(FinalVerts, FinalIndices, &OutFaceRemap, &OutVertexRemap);
			}

			// Build particle list #BG Maybe allow TParticles to copy vectors?
			Chaos::FTriangleMeshImplicitObject::ParticlesType TriMeshParticles;
			TriMeshParticles.AddParticles(FinalVerts.Num());

			const int32 NumVerts = FinalVerts.Num();
			for(int32 VertIndex = 0; VertIndex < NumVerts; ++VertIndex)
			{
				TriMeshParticles.SetX(VertIndex, FinalVerts[VertIndex]);
			}

			// Build chaos triangle list. #BGTODO Just make the clean function take these types instead of double copying
			const int32 NumTriangles = FinalIndices.Num() / 3;
			bool bHasMaterials = InParams.TriangleMeshDesc.MaterialIndices.Num() > 0;
			TArray<uint16> MaterialIndices;

			auto LambdaHelper = [&](auto& Triangles)
			{
				if(bHasMaterials)
				{
					MaterialIndices.Reserve(NumTriangles);
				}

				// Need to rebuild face remap array, in case there are any invalid triangles
				TArray<int32> OldFaceRemap = MoveTemp(OutFaceRemap);
				OutFaceRemap.Reserve(OldFaceRemap.Num());

				Triangles.Reserve(NumTriangles);
				for(int32 TriangleIndex = 0; TriangleIndex < NumTriangles; ++TriangleIndex)
				{
					// Only add this triangle if it is valid
					const int32 BaseIndex = TriangleIndex * 3;
					const bool bIsValidTriangle = Chaos::FConvexBuilder::IsValidTriangle(
						FinalVerts[FinalIndices[BaseIndex]],
						FinalVerts[FinalIndices[BaseIndex + 1]],
						FinalVerts[FinalIndices[BaseIndex + 2]]);

					// TODO: Figure out a proper way to handle this. Could these edges get sewn together? Is this important?
					//if (ensureMsgf(bIsValidTriangle, TEXT("FChaosDerivedDataCooker::BuildTriangleMeshes(): Trimesh attempted cooked with invalid triangle!")));
					if(bIsValidTriangle)
					{
						Triangles.Add(Chaos::TVector<int32, 3>(FinalIndices[BaseIndex], FinalIndices[BaseIndex + 1], FinalIndices[BaseIndex + 2]));
						if (OldFaceRemap.Num())
						{
							OutFaceRemap.Add(OldFaceRemap[TriangleIndex]);
						}

						if(bHasMaterials)
						{
							if(EnableMeshClean)
							{
								if(!ensure(OldFaceRemap.IsValidIndex(TriangleIndex)))
								{
									MaterialIndices.Empty();
									bHasMaterials = false;
								}
								else
								{
									const int32 OriginalIndex = OldFaceRemap[TriangleIndex];

									if(ensure(InParams.TriangleMeshDesc.MaterialIndices.IsValidIndex(OriginalIndex)))
									{
										MaterialIndices.Add(InParams.TriangleMeshDesc.MaterialIndices[OriginalIndex]);
									}
									else
									{
										MaterialIndices.Empty();
										bHasMaterials = false;
									}
								}
							}
							else
							{
								if(ensure(InParams.TriangleMeshDesc.MaterialIndices.IsValidIndex(TriangleIndex)))
								{
									MaterialIndices.Add(InParams.TriangleMeshDesc.MaterialIndices[TriangleIndex]);
								}
								else
								{
									MaterialIndices.Empty();
									bHasMaterials = false;
								}
							}

						}
					}
				}

				
				// Keep the face remap if either related flag is true, to carry it through initial processing. (It will be stripped later from the cook helper's remap array if bSupportFaceRemap is false.)
				bool bNeedFaceRemapOnBVH = InParams.bSupportFaceRemap || InParams.bSupportFaceRemapOnMeshBVH;
				TUniquePtr<TArray<int32>> OutFaceRemapPtr = bNeedFaceRemapOnBVH ? MakeUnique<TArray<int32>>(OutFaceRemap) : nullptr;
				TUniquePtr<TArray<int32>> OutVertexRemapPtr = (InParams.bSupportVertexRemap && Chaos::TriMeshPerPolySupport) ? MakeUnique<TArray<int32>>(OutVertexRemap) : nullptr;
				if (!OutFaceRemapPtr)
				{
					OutFaceRemap.Empty();
				}
				if (!OutVertexRemapPtr)
				{
					OutVertexRemap.Empty();
				}
				Chaos::FTriangleMeshImplicitObject::FBuildSettings MeshBuildSettings;
				MeshBuildSettings.PerFaceBounds = InParams.TriangleMeshDesc.bReducedMemoryRepresentation ? Chaos::FTrimeshBVH::EPerFaceBoundsRepresentation::Byte : Chaos::FTrimeshBVH::EPerFaceBoundsRepresentation::VectorizedFloat;
				constexpr bool bCullsBackFaceRaycast = false;
				Chaos::FTriangleMeshImplicitObject* TriangleMesh = new Chaos::FTriangleMeshImplicitObject(MoveTemp(TriMeshParticles), MoveTemp(Triangles), MoveTemp(MaterialIndices), MoveTemp(OutFaceRemapPtr), MoveTemp(OutVertexRemapPtr),
					bCullsBackFaceRaycast, MeshBuildSettings);
				OutTriangleMeshes.Emplace(TriangleMesh);

				// Propagate remapped indices from the FTriangleMeshImplicitObject back to the remap array
				if (bNeedFaceRemapOnBVH)
				{
					CopyUpdatedFaceRemapFromTriangleMesh(*TriangleMesh, OutFaceRemap);
				}
			};

			if(FinalVerts.Num() < TNumericLimits<uint16>::Max())
			{
				TArray<Chaos::TVector<uint16, 3>> TrianglesSmallIdx;
				LambdaHelper(TrianglesSmallIdx);
			}
			else
			{
				TArray<Chaos::TVector<int32, 3>> TrianglesLargeIdx;
				LambdaHelper(TrianglesLargeIdx);
			}
		}
	}

	FCookHelper::FCookHelper(UBodySetup* InSetup)
		: SourceSetup(InSetup)
		, bCanceled(false)
	{
		check(SourceSetup);
		EPhysXMeshCookFlags TempFlags = static_cast<EPhysXMeshCookFlags>(0);
		SourceSetup->GetCookInfo(CookInfo, TempFlags);
	}

	void FCookHelper::Cook()
	{
		if (bCanceled) return;

		Cooking::BuildConvexMeshes(SimpleImplicits, CookInfo);
		Cooking::BuildTriangleMeshes(ComplexImplicits, FaceRemap, VertexRemap, CookInfo);

		if(CookInfo.bSupportUVFromHitResults)
		{
			UVInfo.FillFromTriMesh(CookInfo.TriangleMeshDesc);
		}

		if(!CookInfo.bSupportFaceRemap)
		{
			FaceRemap.Empty();
		}
	}

	void FCookHelper::CookAsync(FSimpleDelegateGraphTask::FDelegate CompletionDelegate)
	{
		Cook();
		FSimpleDelegateGraphTask::CreateAndDispatchWhenReady(CompletionDelegate, GET_STATID(STAT_PhysXCooking), nullptr, ENamedThreads::GameThread);
	}

	bool FCookHelper::HasWork() const
	{
		return CookInfo.bCookTriMesh || CookInfo.bCookNonMirroredConvex || CookInfo.bCookMirroredConvex;
	}
}
