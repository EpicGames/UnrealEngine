// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosDerivedData.h"

#include "Chaos/TriangleMeshImplicitObject.h"

#include "Serialization/MemoryWriter.h"
#include "PhysicsEngine/Experimental/ChaosCooking.h"

#if WITH_EDITOR
#include "DerivedDataCacheKey.h"
#endif

class FChaosDerivedDataCookerRefHolder : public FGCObject
{
public:
	FChaosDerivedDataCookerRefHolder(FChaosDerivedDataCooker* InCooker)
		: Cooker(InCooker)
	{
	}

	// FGCObject Interface
	virtual void AddReferencedObjects(FReferenceCollector& Collector) override
	{
		if (Cooker && Cooker->Setup)
		{
			Collector.AddReferencedObject(Cooker->Setup);
		}
	}
	virtual FString GetReferencerName() const override
	{
		return TEXT("ChaosDerivedDataCooker");
	}
	// End FGCObject Interface

private:
	FChaosDerivedDataCooker* Cooker;
};

const TCHAR* FChaosDerivedDataCooker::GetPluginName() const
{
#if WITH_EDITOR
	static UE::DerivedData::FCacheBucket LegacyBucket(TEXTVIEW("LegacyChaosGeometryData"), TEXTVIEW("BodySetup"));
#endif
	return TEXT("ChaosGeometryData");
}

const TCHAR* FChaosDerivedDataCooker::GetVersionString() const
{
	// As changing our DDC version will most likely affect any external callers that rely on Chaos types
	// for their own DDC or serialized data - change Chaos::ChaosVersionString in Chaos/Core.h to bump our
	// Chaos data version. Callers can also rely on that version in their builders and avoid bad serialization
	// when basic Chaos data changes
	return Chaos::ChaosVersionGUID;
}

FString FChaosDerivedDataCooker::GetDebugContextString() const
{
	if (Setup)
	{
		UObject* Outer = Setup->GetOuter();
		if (Outer)
		{
			return Outer->GetFullName();
		}
	}

	return FDerivedDataPluginInterface::GetDebugContextString();
}

FString FChaosDerivedDataCooker::GetPluginSpecificCacheKeySuffix() const
{
	GatherKeyData();
	return PluginSpecificCacheKeySuffix;
}

void FChaosDerivedDataCooker::GatherKeyData() const
{
	if (!PluginSpecificCacheKeySuffix.IsEmpty())
	{
		return;
	}

	FString SetupGeometryKey("INVALID");

	if (Setup)
	{
		Setup->GetGeometryDDCKey(SetupGeometryKey);
	}

	PluginSpecificCacheKeySuffix = FString::Printf(TEXT("%s_%s_REAL%d"),
										*RequestedFormat.ToString(),
										*SetupGeometryKey,
										(int)sizeof(Chaos::FReal));

#if PLATFORM_CPU_ARM_FAMILY
	// Separate out arm keys as x64 and arm64 clang do not generate the same data for a given
	// input. Add the arm specifically so that a) we avoid rebuilding the current DDC and
	// b) we can remove it once we get arm64 to be consistent.
	PluginSpecificCacheKeySuffix.Append(TEXT("_arm64"));
#endif
}

bool FChaosDerivedDataCooker::IsBuildThreadsafe() const
{
	// #BG Investigate Parallel Build
	return false;
}

bool FChaosDerivedDataCooker::Build(TArray<uint8>& OutData)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FChaosDerivedDataCooker::Build);

	bool bSucceeded = false;

	if(Setup)
	{
		FMemoryWriter MemWriterAr(OutData);
		Chaos::FChaosArchive Ar(MemWriterAr);

		int32 PrecisionSize = (int32)sizeof(BuildPrecision);

		Ar << PrecisionSize;

		Chaos::FCookHelper Cooker(Setup);
		Cooker.Cook();

		Ar << Cooker.SimpleImplicits << Cooker.ComplexImplicits << Cooker.UVInfo << Cooker.FaceRemap;

		bSucceeded = true;
	}

	return bSucceeded;
}

FChaosDerivedDataCooker::FChaosDerivedDataCooker(UBodySetup* InSetup, FName InFormat, bool bInUseRefHolder)
	: Setup(InSetup)
	, RequestedFormat(InFormat)
{
	if (bInUseRefHolder)
	{
		RefHolder = MakeUnique<FChaosDerivedDataCookerRefHolder>(this);
	}
}


