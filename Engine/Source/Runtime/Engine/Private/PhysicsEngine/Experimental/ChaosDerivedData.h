// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "DerivedDataPluginInterface.h"
#include "UObject/GCObject.h"
#include "Interface_CollisionDataProviderCore.h"
#include "Templates/UniquePtr.h"
#include "UObject/ObjectPtr.h"

struct FBodySetupUVInfo;
struct FCookBodySetupInfo;

class UBodySetup;

namespace Chaos
{
	class FChaosArchive;

	class FImplicitObject;

	class FTriangleMeshImplicitObject;
}

struct FChaosTriMeshCollisionBuildParameters
{
	bool bCollapseVerts;

	FTriMeshCollisionData MeshDesc;
	FBodySetupUVInfo* UVInfo;
};

struct FChaosConvexMeshCollisionBuildParameters
{
	bool bCollapseVerts;
	bool bMirror;
};

class FChaosDerivedDataCooker : public FDerivedDataPluginInterface
{
public:

	using BuildPrecision = float;

	// FDerivedDataPluginInterface Interface
	virtual const TCHAR* GetPluginName() const override;
	virtual const TCHAR* GetVersionString() const override;
	virtual FString GetPluginSpecificCacheKeySuffix() const override;
	virtual bool IsBuildThreadsafe() const override;
	virtual bool Build(TArray<uint8>& OutData) override;
	virtual FString GetDebugContextString() const override;
	//End FDerivedDataPluginInterface Interface

	FChaosDerivedDataCooker(UBodySetup* InSetup, FName InFormat, bool bUseRefHolder = true);
	void GatherKeyData() const;

	bool CanBuild()
	{
		return Setup != nullptr;
	}

private:
	friend class FChaosDerivedDataCookerRefHolder;
	TObjectPtr<UBodySetup> Setup;
	FName RequestedFormat;
	TUniquePtr<FGCObject> RefHolder;
	mutable FString PluginSpecificCacheKeySuffix;
};


