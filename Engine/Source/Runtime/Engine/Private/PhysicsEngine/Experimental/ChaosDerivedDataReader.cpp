// Copyright Epic Games, Inc. All Rights Reserved.

#include "Physics/Experimental/ChaosDerivedDataReader.h"

#include "Chaos/CollisionConvexMesh.h"
#include "Chaos/TriangleMeshImplicitObject.h"
#include "Chaos/Convex.h"

template<typename T, int d>
FChaosDerivedDataReader<T, d>::FChaosDerivedDataReader(FBulkData* InBulkData)
	: bReadSuccessful(false)
{
	const int32 DataTypeSize = sizeof(T);

	uint8* DataPtr = (uint8*)InBulkData->LockReadOnly();
	FBufferReader Ar(DataPtr, InBulkData->GetBulkDataSize(), false);
	Chaos::FChaosArchive ChaosAr(Ar);

	int32 SerializedDataSize = -1;

	ChaosAr << SerializedDataSize;
	Ar.UsingCustomVersion(FFortniteValkyrieBranchObjectVersion::GUID);
	

	if(SerializedDataSize != DataTypeSize)
	{
		// Can't use this data, it was serialized for a different precision 
		ensureMsgf(false, TEXT("Failed to load Chaos body setup bulk data. Expected fp precision to be width %d but it was %d"), DataTypeSize, SerializedDataSize);
	}
	else
	{
		{
			LLM_SCOPE(ELLMTag::ChaosConvex);
			ChaosAr << ConvexGeometries;
		}

		{
			LLM_SCOPE(ELLMTag::ChaosTrimesh);
			ChaosAr << TriMeshGeometries << UVInfo << FaceRemap;
		}

		bReadSuccessful = true;
	}

	InBulkData->Unlock();
}

template class FChaosDerivedDataReader<float, 3>;
