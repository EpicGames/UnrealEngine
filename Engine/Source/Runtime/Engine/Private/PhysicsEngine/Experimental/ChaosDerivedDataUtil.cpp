// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosDerivedDataUtil.h"

#include "Chaos/AABBTree.h"

namespace Chaos
{

	struct FCleanMeshWrapper
	{
		FVector Vert;

		template <typename TPayloadType>
		TPayloadType GetPayload(int32 Idx) const { return Idx; }

		bool HasBoundingBox() const { return true; }
		FAABB3 BoundingBox() const { return FAABB3(Vert, Vert); }
	};

	void CleanTrimesh(TArray<FVector3f>& InOutVertices, TArray<int32>& InOutIndices, TArray<int32>* OutOptFaceRemap, TArray<int32>* OutOptVertexRemap)
	{
		QUICK_SCOPE_CYCLE_COUNTER(STATQ_CleanTrimesh);
		TArray<FVector3f> LocalSourceVerts = InOutVertices;
		TArray<int32> LocalSourceIndices = InOutIndices;
		
		const int32 NumSourceVerts = LocalSourceVerts.Num();
		const int32 NumSourceTriangles = LocalSourceIndices.Num() / 3;

		if(NumSourceVerts == 0 || (LocalSourceIndices.Num() % 3) != 0)
		{
			// No valid geometry passed in
			return;
		}

		// New condensed list of unique verts from the trimesh
		TArray<FVector3f> LocalUniqueVerts;
		// New condensed list of indices after cleaning
		TArray<int32> LocalUniqueIndices;
		// Array mapping unique vertex index back to source data index
		TArray<int32> LocalUniqueToSourceIndices;
		// Remapping table from source index to unique index
		TArray<int32> LocalVertexRemap;
		
		LocalUniqueVerts.Reserve(NumSourceVerts);
		LocalVertexRemap.AddUninitialized(NumSourceVerts);

		auto ValidateTrianglesPre = [&InOutVertices](int32 A, int32 B, int32 C) -> bool
		{
			const FVector3f v0 = InOutVertices[A];
			const FVector3f v1 = InOutVertices[B];
			const FVector3f v2 = InOutVertices[C];
			return v0 != v1 && v0 != v2 && v1 != v2;
		};

		int32 NumBadTris = 0;
		{
			QUICK_SCOPE_CYCLE_COUNTER(STATQ_Validate);
			for(int32 SrcTriIndex = 0; SrcTriIndex < NumSourceTriangles; ++SrcTriIndex)
			{
				const int32 A = InOutIndices[SrcTriIndex * 3];
				const int32 B = InOutIndices[SrcTriIndex * 3 + 1];
				const int32 C = InOutIndices[SrcTriIndex * 3 + 2];

				if(!ValidateTrianglesPre(A, B, C))
				{
					++NumBadTris;
				}
			}
		}
		UE_CLOGF(NumBadTris > 0, LogChaos, Display, "Input trimesh contains %d bad triangles.", NumBadTris);

		float WeldThresholdSq = 0.0f;// SMALL_NUMBER * SMALL_NUMBER;

		TArray<FCleanMeshWrapper> WrapperVerts;

		for (int i = 0; i < LocalSourceVerts.Num(); ++i)
		{
			WrapperVerts.Add(FCleanMeshWrapper{ (FVector)LocalSourceVerts[i] });
		}

		TAABBTree<int32, TAABBTreeLeafArray<int32>> Accel(WrapperVerts);
		TSet<int32> NonUnique;

		{
			QUICK_SCOPE_CYCLE_COUNTER(STATQ_FindDuplicates);
			for(int32 SourceVertIndex = 0; SourceVertIndex < NumSourceVerts; ++SourceVertIndex)
			{
				if(NonUnique.Contains(SourceVertIndex))
				{
					continue;
				}

				const FVector3f& SourceVert = LocalSourceVerts[SourceVertIndex];

				TArray<int32> Duplicates = Accel.FindAllIntersections(FAABB3(SourceVert - WeldThresholdSq, SourceVert + WeldThresholdSq));
				ensure(Duplicates.Num() > 0);	//Should always find at least original vert

				//first index is always considered unique
				LocalUniqueVerts.Add(SourceVert);
				LocalUniqueToSourceIndices.Add(SourceVertIndex);
				LocalVertexRemap[SourceVertIndex] = LocalUniqueVerts.Num() - 1;


				for(int32 Idx : Duplicates)
				{
					if(Idx != SourceVertIndex)
					{
						ensure(Idx > SourceVertIndex);	//shouldn't be here if a smaller idx already found these duplicates
						NonUnique.Add(Idx);
						LocalVertexRemap[Idx] = LocalVertexRemap[SourceVertIndex];
					}
				}
			}
		}

		// Build the new index buffer, removing now invalid merged triangles
		auto ValidateTriangleIndices = [](int32 A, int32 B, int32 C) -> bool
		{
			return A != B && A != C && B != C;
		};

		auto ValidateTriangleArea = [](const FVector3f& A, const FVector3f& B, const FVector3f& C)
		{
			const float AreaSq = FVector3f::CrossProduct(A - B, A - C).SizeSquared();

			return AreaSq > UE_SMALL_NUMBER;
		};

		int32 NumDiscardedTriangles_Welded = 0;
		int32 NumDiscardedTriangles_Area = 0;

		// Remapping table from source triangle to unique triangle
		TArray<int32> LocalTriangleRemap;
		LocalTriangleRemap.Reserve(NumSourceTriangles);

		{
			QUICK_SCOPE_CYCLE_COUNTER(STATQ_Remap);
			for(int32 OriginalTriIndex = 0; OriginalTriIndex < NumSourceTriangles; ++OriginalTriIndex)
			{
				const int32 OrigAIndex = LocalSourceIndices[OriginalTriIndex * 3];
				const int32 OrigBIndex = LocalSourceIndices[OriginalTriIndex * 3 + 1];
				const int32 OrigCIndex = LocalSourceIndices[OriginalTriIndex * 3 + 2];

				const int32 RemappedAIndex = LocalVertexRemap[OrigAIndex];
				const int32 RemappedBIndex = LocalVertexRemap[OrigBIndex];
				const int32 RemappedCIndex = LocalVertexRemap[OrigCIndex];

				const FVector3f& A = LocalUniqueVerts[RemappedAIndex];
				const FVector3f& B = LocalUniqueVerts[RemappedBIndex];
				const FVector3f& C = LocalUniqueVerts[RemappedCIndex];

				// Only consider triangles that are actually valid for collision
				// #BG Consider being able to fix small triangles by collapsing them if we hit this a lot
				const bool bValidIndices = ValidateTriangleIndices(RemappedAIndex, RemappedBIndex, RemappedCIndex);
				const bool bValidArea = ValidateTriangleArea(A, B, C);
				if(bValidIndices && bValidArea)
				{
					LocalUniqueIndices.Add(RemappedAIndex);
					LocalUniqueIndices.Add(RemappedBIndex);
					LocalUniqueIndices.Add(RemappedCIndex);
					LocalTriangleRemap.Add(OriginalTriIndex);
				}
				else
				{
					if(!bValidIndices)
					{
						++NumDiscardedTriangles_Welded;
					}
					else if(!bValidArea)
					{
						++NumDiscardedTriangles_Area;
					}
				}
			}
		}

		CHAOS_CLOG(NumDiscardedTriangles_Welded > 0, LogChaos, Warning, TEXT("Discarded %d welded triangles when cooking trimesh."), NumDiscardedTriangles_Welded);
		CHAOS_CLOG(NumDiscardedTriangles_Area > 0, LogChaos, Warning, TEXT("Discarded %d small triangles when cooking trimesh."), NumDiscardedTriangles_Area);

		InOutVertices = LocalUniqueVerts;
		InOutIndices = LocalUniqueIndices;

		if(OutOptFaceRemap)
		{
			*OutOptFaceRemap = LocalTriangleRemap;
		}
		if (OutOptVertexRemap)
		{
			*OutOptVertexRemap = LocalVertexRemap;
		}
	}

}
