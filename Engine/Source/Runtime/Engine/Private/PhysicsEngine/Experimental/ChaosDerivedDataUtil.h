// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

namespace Chaos
{
	void CleanTrimesh(TArray<FVector3f>& InOutVertices, TArray<int32>& InOutIndices, TArray<int32>* OutOptFaceRemap, TArray<int32>* OutOptVertexRemap);
}
