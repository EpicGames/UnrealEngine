// Copyright Epic Games, Inc. All Rights Reserved.

#include "Physics/Experimental/ChaosEventRelay.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ChaosEventRelay)

UChaosEventRelay::UChaosEventRelay()
{
}

void UChaosEventRelay::DispatchPhysicsCollisionEvents(const TArray<FCollisionChaosEvent>& CollisionEvents)
{
	
	if (OnCollisionEvent.IsBound())
	{
		OnCollisionEvent.Broadcast(CollisionEvents);
	}
}

void UChaosEventRelay::DispatchPhysicsBreakEvents(const TArray<FChaosBreakEvent>& BreakEvents)
{
	if (OnBreakEvent.IsBound())
	{
		OnBreakEvent.Broadcast(BreakEvents);
	}
}

void UChaosEventRelay::DispatchPhysicsRemovalEvents(const TArray<FChaosRemovalEvent>& RemovalEvents)
{
	if (OnRemovalEvent.IsBound())
	{
		OnRemovalEvent.Broadcast(RemovalEvents);
	}
}

void UChaosEventRelay::DispatchPhysicsCrumblingEvents(const TArray<FChaosCrumblingEvent>& CrumblingEvents)
{
	if (OnCrumblingEvent.IsBound())
	{
		OnCrumblingEvent.Broadcast(CrumblingEvents);
	}
}

void UChaosEventRelay::DispatchPhysicsTrailingEvents(const TArray<FChaosTrailingEvent>& TrailingEvents)
{
	if (OnTrailingEvent.IsBound())
	{
		OnTrailingEvent.Broadcast(TrailingEvents);
	}
}
