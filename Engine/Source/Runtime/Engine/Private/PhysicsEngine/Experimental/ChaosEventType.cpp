// Copyright Epic Games, Inc. All Rights Reserved.

#include "Physics/Experimental/ChaosEventType.h"
#include "Chaos/ExternalCollisionData.h"


#include UE_INLINE_GENERATED_CPP_BY_NAME(ChaosEventType)

FCollisionChaosEventBodyInfo::FCollisionChaosEventBodyInfo()
	: Velocity(FVector::ZeroVector)
	, DeltaVelocity(FVector::ZeroVector)
	, AngularVelocity(FVector::ZeroVector)
	, Mass(0.0)
	, PhysMaterial(nullptr)
	, Component(nullptr)
	, BodyIndex(0)
	, BoneName(NAME_None)
{
}

FCollisionChaosEvent::FCollisionChaosEvent()
	: Location(FVector::ZeroVector)
	, AccumulatedImpulse(FVector::ZeroVector)
	, Normal(FVector::ZeroVector)
	, PenetrationDepth(0.0)
{
}

FCollisionChaosEvent::FCollisionChaosEvent(const Chaos::FCollidingData& CollisionData)
	: Location(CollisionData.Location)
	, AccumulatedImpulse(CollisionData.AccumulatedImpulse)
	, Normal(CollisionData.Normal)
	, PenetrationDepth(CollisionData.PenetrationDepth)
{
	Body1.Velocity = CollisionData.Velocity1;
	Body1.DeltaVelocity = CollisionData.DeltaVelocity1;
	Body1.AngularVelocity = CollisionData.AngularVelocity1;
	Body1.Mass = CollisionData.Mass1;

	Body2.Velocity = CollisionData.Velocity2;
	Body2.DeltaVelocity = CollisionData.DeltaVelocity2;
	Body2.AngularVelocity = CollisionData.AngularVelocity2;
	Body2.Mass = CollisionData.Mass2;

}

FChaosBreakEvent::FChaosBreakEvent()
	: Component(nullptr)
	, Location(FVector::ZeroVector)
	, Orientation(FQuat::Identity)
	, Velocity(FVector::ZeroVector)
	, AngularVelocity(FVector::ZeroVector)
	, Extents(FVector::ZeroVector)
	, Mass(0.0f)
	, Index(INDEX_NONE)
	, bFromCrumble(false)
{
}

FChaosBreakEvent::FChaosBreakEvent(const Chaos::FBreakingData& BreakingData)
	: Component(nullptr)
	, Location(BreakingData.Location)
	, Orientation(BreakingData.Orientation)
	, Velocity(BreakingData.Velocity)
	, AngularVelocity(BreakingData.AngularVelocity)
	, Extents(BreakingData.BoundingBox.Extents())
	, Mass(BreakingData.Mass)
	, Index(BreakingData.TransformGroupIndex)
	, bFromCrumble(BreakingData.bFromCrumble)
{
}

FChaosRemovalEvent::FChaosRemovalEvent()
	: Component(nullptr)
	, Location(FVector::ZeroVector)
	, Mass(0.0f)
{
}
