// Copyright Epic Games, Inc. All Rights Reserved.

#include "Physics/Experimental/ChaosInterfaceUtils.h"

#include "BodySetupEnums.h"
#include "Chaos/Capsule.h"
#include "Chaos/CollisionConvexMesh.h"
#include "Chaos/Convex.h"
#include "Chaos/TriangleMeshImplicitObject.h"

#include "Physics/PhysicsInterfaceTypes.h"

#include "PhysicsEngine/AggregateGeom.h"
#include "PhysicsEngine/BoxElem.h"
#include "PhysicsEngine/ConvexElem.h"
#include "PhysicsEngine/PhysicsSettings.h"
#include "PhysicsEngine/SphylElem.h"
#include "PhysicsEngine/SphereElem.h"
#include "PhysicsEngine/TaperedCapsuleElem.h"

#define FORCE_ANALYTICS 0
#define CREATE_STRAIGHT_CAPSULE_GEOMETRY_FOR_TAPERED_CAPSULES

static_assert(sizeof(Chaos::TImplicitObjectTransformed<Chaos::FReal, 3>) <= 192, 
	"TImplicitObjectTransformed was optimized to fit into MB3 192 bin to reduce excess memory waste");

namespace ChaosInterface
{
	float Chaos_Collision_MarginFraction = -1.0f;
	FAutoConsoleVariableRef CVarChaosCollisionMarginFraction(TEXT("p.Chaos.Collision.MarginFraction"), Chaos_Collision_MarginFraction, TEXT("Override the collision margin fraction set in Physics Settings (if >= 0)"));

	float Chaos_Collision_MarginMax = -1.0f;
	FAutoConsoleVariableRef CVarChaosCollisionMarginMax(TEXT("p.Chaos.Collision.MarginMax"), Chaos_Collision_MarginMax, TEXT("Override the max collision margin set in Physics Settings (if >= 0)"));


	template<class PHYSX_MESH>
	TArray<Chaos::TVec3<int32>> GetMeshElements(const PHYSX_MESH* PhysXMesh)
	{
		check(false);
	}

	Chaos::EChaosCollisionTraceFlag ConvertCollisionTraceFlag(ECollisionTraceFlag Flag)
	{
		if (Flag == ECollisionTraceFlag::CTF_UseDefault)
			return Chaos::EChaosCollisionTraceFlag::Chaos_CTF_UseDefault;
		if (Flag == ECollisionTraceFlag::CTF_UseSimpleAndComplex)
			return Chaos::EChaosCollisionTraceFlag::Chaos_CTF_UseSimpleAndComplex;
		if (Flag == ECollisionTraceFlag::CTF_UseSimpleAsComplex)
			return Chaos::EChaosCollisionTraceFlag::Chaos_CTF_UseSimpleAsComplex;
		if (Flag == ECollisionTraceFlag::CTF_UseComplexAsSimple)
			return Chaos::EChaosCollisionTraceFlag::Chaos_CTF_UseComplexAsSimple;
		if (Flag == ECollisionTraceFlag::CTF_MAX)
			return Chaos::EChaosCollisionTraceFlag::Chaos_CTF_MAX;
		ensure(false);
		return Chaos::EChaosCollisionTraceFlag::Chaos_CTF_UseDefault;
	}
	
	void CreateGeometry(const FGeometryAddParams& InParams, TArray<TUniquePtr<Chaos::FImplicitObject>>& OutGeoms, Chaos::FShapesArray& OutShapes)
	{
		check(false);
	}

	void CreateGeometry(const FGeometryAddParams& InParams, TArray<Chaos::FImplicitObjectPtr>& OutGeoms, Chaos::FShapesArray& OutShapes)
	{
		LLM_SCOPE(ELLMTag::ChaosGeometry);
		const FVector& Scale = InParams.Scale;
		TArray<Chaos::FImplicitObjectPtr>& Geoms = OutGeoms;
		Chaos::FShapesArray& Shapes = OutShapes;

		ECollisionTraceFlag CollisionTraceType = InParams.CollisionTraceType;
		if (CollisionTraceType == CTF_UseDefault)
		{
			CollisionTraceType = UPhysicsSettings::Get()->DefaultShapeComplexity;
		}

		const FChaosSolverConfiguration& SolverOptions = UPhysicsSettingsCore::Get()->SolverOptions;
		float CollisionMarginFraction = FMath::Max(0.0f, SolverOptions.CollisionMarginFraction);
		float CollisionMarginMax = FMath::Max(0.0f, SolverOptions.CollisionMarginMax);

		// Test margins without changing physics settings
		if (Chaos_Collision_MarginFraction >= 0.0f)
		{
			CollisionMarginFraction = Chaos_Collision_MarginFraction;
		}
		if (Chaos_Collision_MarginMax >= 0.0f)
		{
			CollisionMarginMax = Chaos_Collision_MarginMax;
		}

		// Complex as simple should not create simple geometry, unless there is no complex geometry.  Otherwise both get queried against.
		const bool bMakeSimpleGeometry = (CollisionTraceType != CTF_UseComplexAsSimple) || (InParams.TriMeshGeometries.Num() == 0);

		// The reverse is true for Simple as Complex.
		const int32 SimpleShapeCount = InParams.Geometry->SphereElems.Num() + InParams.Geometry->BoxElems.Num() + InParams.Geometry->SphylElems.Num() + InParams.Geometry->TaperedCapsuleElems.Num() + InParams.Geometry->ConvexElems.Num();
		const bool bMakeComplexGeometry = (CollisionTraceType != CTF_UseSimpleAsComplex) || (SimpleShapeCount == 0);

		ensure(bMakeComplexGeometry || bMakeSimpleGeometry);

		const int32 NumNewShapes = (bMakeSimpleGeometry ? SimpleShapeCount : 0) + (bMakeComplexGeometry ? InParams.TriMeshGeometries.Num() : 0);
		Shapes.Reserve(Shapes.Num() + NumNewShapes);
		Geoms.Reserve(Geoms.Num() + NumNewShapes);

		auto NewShapeHelper = [&InParams, &CollisionTraceType](const Chaos::FImplicitObjectPtr& InGeom, int32 ShapeIdx, void* UserData, ECollisionEnabled::Type ShapeCollisionEnabled, bool bComplexShape = false)
		{
			TUniquePtr<Chaos::FPerShapeData> NewShape = Chaos::FShapeInstanceProxy::Make(ShapeIdx, InGeom);
			const FBodyCollisionFilterData& CollisionFilterData = InParams.CollisionData.CollisionFilterData;
			const Chaos::Filter::FShapeFilterData& ShapeFilterData = bComplexShape ? CollisionFilterData.ComplexShapeFilterData : CollisionFilterData.SimpleShapeFilterData;
			NewShape->SetShapeFilterData(ShapeFilterData);
			NewShape->SetFilterInstanceData(CollisionFilterData.FilterInstanceData);
			NewShape->SetCollisionTraceType(ConvertCollisionTraceFlag(CollisionTraceType));
			NewShape->UpdateShapeBounds(InParams.WorldTransform);
			NewShape->SetUserData(UserData);

			// Combine shape and body collision-enabled using the same method as UpdatePhysicsFilterData,
			// but with the added nastiness of needing collision mode represented as bool flags.
			bool bEnableSim = InParams.CollisionData.CollisionFlags.bEnableSimCollisionSimple || InParams.CollisionData.CollisionFlags.bEnableSimCollisionComplex;
			bool bEnableQuery = InParams.CollisionData.CollisionFlags.bEnableQueryCollision;
			bool bEnableProbe = InParams.CollisionData.CollisionFlags.bEnableProbeCollision;
			const ECollisionEnabled::Type BodyCollisionEnabled = CollisionEnabledFromFlags(bEnableQuery, bEnableSim, bEnableProbe);
			const ECollisionEnabled::Type CombinedCollisionEnabled = CollisionEnabledIntersection(BodyCollisionEnabled, ShapeCollisionEnabled);
			CollisionEnabledToFlags(CombinedCollisionEnabled, bEnableQuery, bEnableSim, bEnableProbe);

			// Mirror behavior of FBodyInstance::BuildBodyCollisionFlags.
			// At the chaos level, if probe is enabled, we must also have sim.
			bEnableSim |= bEnableProbe;

			// Set the parameters on the shape
			NewShape->SetSimEnabled(bEnableSim);
			NewShape->SetQueryEnabled(bEnableQuery);
			NewShape->SetIsProbe(bEnableProbe);

			return NewShape;
		};

		if (bMakeSimpleGeometry)
		{
			for (const FKSphereElem& SphereElem : InParams.Geometry->SphereElems)
			{
				const FKSphereElem ScaledSphereElem = SphereElem.GetFinalScaled(Scale, InParams.LocalTransform);
				const float UseRadius = FMath::Max(ScaledSphereElem.Radius, UE_KINDA_SMALL_NUMBER);
				auto ImplicitSphere = MakeImplicitObjectPtr<Chaos::TSphere<Chaos::FReal, 3>>(ScaledSphereElem.Center, UseRadius);
				TUniquePtr<Chaos::FPerShapeData> NewShape = NewShapeHelper(ImplicitSphere, Shapes.Num(), (void*)SphereElem.GetUserData(), SphereElem.GetCollisionEnabled());
				Shapes.Emplace(MoveTemp(NewShape));
				Geoms.Emplace(MoveTemp(ImplicitSphere));
			}
			for (const FKBoxElem& BoxElem : InParams.Geometry->BoxElems)
			{
				const FKBoxElem ScaledBoxElem = BoxElem.GetFinalScaled(Scale, InParams.LocalTransform);
				const FTransform& BoxTransform = ScaledBoxElem.GetTransform();
				Chaos::FVec3 HalfExtents = Chaos::FVec3(ScaledBoxElem.X * 0.5f, ScaledBoxElem.Y * 0.5f, ScaledBoxElem.Z * 0.5f);

				HalfExtents.X = FMath::Max(HalfExtents.X, UE_KINDA_SMALL_NUMBER);
				HalfExtents.Y = FMath::Max(HalfExtents.Y, UE_KINDA_SMALL_NUMBER);
				HalfExtents.Z = FMath::Max(HalfExtents.Z, UE_KINDA_SMALL_NUMBER);

				const Chaos::FReal CollisionMargin = FMath::Min(2.0f * HalfExtents.GetMin() * CollisionMarginFraction, CollisionMarginMax);

				// AABB can handle translations internally but if we have a rotation we need to wrap it in a transform
				Chaos::FImplicitObjectPtr Implicit;
				if (!BoxTransform.GetRotation().IsIdentity())
				{
					auto ImplicitBox = MakeImplicitObjectPtr<Chaos::TBox<Chaos::FReal, 3>>(-HalfExtents, HalfExtents, CollisionMargin);
					Implicit = MakeImplicitObjectPtr<Chaos::TImplicitObjectTransformed<Chaos::FReal, 3>>(MoveTemp(ImplicitBox), BoxTransform);
				}
				else
				{
					Implicit = MakeImplicitObjectPtr<Chaos::TBox<Chaos::FReal, 3>>(BoxTransform.GetTranslation() - HalfExtents, BoxTransform.GetTranslation() + HalfExtents, CollisionMargin);
				}

				TUniquePtr<Chaos::FPerShapeData> NewShape = NewShapeHelper(Implicit,Shapes.Num(), (void*)BoxElem.GetUserData(), BoxElem.GetCollisionEnabled());
				Shapes.Emplace(MoveTemp(NewShape));
				Geoms.Emplace(MoveTemp(Implicit));
			}
			for (const FKSphylElem& UnscaledSphyl : InParams.Geometry->SphylElems)
			{
				const FKSphylElem ScaledSphylElem = UnscaledSphyl.GetFinalScaled(Scale, InParams.LocalTransform);
				Chaos::FReal HalfHeight = FMath::Max(ScaledSphylElem.Length * 0.5f, UE_KINDA_SMALL_NUMBER);
				const Chaos::FReal Radius = FMath::Max(ScaledSphylElem.Radius, UE_KINDA_SMALL_NUMBER);

				Chaos::FImplicitObjectPtr Object;
				if (HalfHeight < UE_KINDA_SMALL_NUMBER)
				{
					//not a capsule just use a sphere
					Object = MakeImplicitObjectPtr<Chaos::TSphere<Chaos::FReal, 3>>(ScaledSphylElem.Center, Radius);
				}
				else
				{
					Chaos::FVec3 HalfExtents = ScaledSphylElem.Rotation.RotateVector(Chaos::FVec3(0, 0, HalfHeight));
					Object = MakeImplicitObjectPtr<Chaos::FCapsule>(ScaledSphylElem.Center - HalfExtents, ScaledSphylElem.Center + HalfExtents, Radius);
				}

				TUniquePtr<Chaos::FPerShapeData> NewShape = NewShapeHelper(Object, Shapes.Num(), (void*)UnscaledSphyl.GetUserData(), UnscaledSphyl.GetCollisionEnabled());
				Shapes.Emplace(MoveTemp(NewShape));
				Geoms.Emplace(MoveTemp(Object));
			}
#ifdef CREATE_STRAIGHT_CAPSULE_GEOMETRY_FOR_TAPERED_CAPSULES
			for (const FKTaperedCapsuleElem& UnscaledTaperedCapsule : InParams.Geometry->TaperedCapsuleElems)
			{
				const FKTaperedCapsuleElem ScaledTaperedCapsule = UnscaledTaperedCapsule.GetFinalScaled(Scale, InParams.LocalTransform);
				Chaos::FReal HalfHeight = FMath::Max(ScaledTaperedCapsule.Length * 0.5f, UE_KINDA_SMALL_NUMBER);
				const Chaos::FReal Radius0 = FMath::Max(ScaledTaperedCapsule.Radius0, UE_KINDA_SMALL_NUMBER);
				const Chaos::FReal Radius1 = FMath::Max(ScaledTaperedCapsule.Radius1, UE_KINDA_SMALL_NUMBER);

				Chaos::FImplicitObjectPtr Object;
				if (HalfHeight < UE_KINDA_SMALL_NUMBER)
				{
					//not a capsule just use a sphere
					const Chaos::FReal MaxRadius = FMath::Max(Radius0, Radius1);
					Object = MakeImplicitObjectPtr<Chaos::TSphere<Chaos::FReal, 3>>(ScaledTaperedCapsule.Center, MaxRadius);
				}
				else
				{
					Chaos::FVec3 HalfExtents = ScaledTaperedCapsule.Rotation.RotateVector(Chaos::FVec3(0, 0, HalfHeight));
					const Chaos::FReal MeanRadius = 0.5f * (Radius0 + Radius1);
					Object = MakeImplicitObjectPtr<Chaos::FCapsule>(ScaledTaperedCapsule.Center - HalfExtents, ScaledTaperedCapsule.Center + HalfExtents, MeanRadius);
				}

				TUniquePtr<Chaos::FPerShapeData> NewShape = NewShapeHelper(Object, Shapes.Num(), (void*)UnscaledTaperedCapsule.GetUserData(), UnscaledTaperedCapsule.GetCollisionEnabled());
				Shapes.Emplace(MoveTemp(NewShape));
				Geoms.Emplace(MoveTemp(Object));
			}
#elif 0
			for (const FKTaperedCapsuleElem& TaperedCapsule : InParams.Geometry->TaperedCapsuleElems)
			{
				ensure(FMath::IsNearlyEqual(Scale[0], Scale[1]) && FMath::IsNearlyEqual(Scale[1], Scale[2]));
				if (TaperedCapsule.Length == 0)
				{
					Chaos::TSphere<FReal, 3>* ImplicitSphere = new Chaos::TSphere<FReal, 3>(-half_extents, TaperedCapsule.Radius * Scale[0]);
					if (PhysicsProxy) PhysicsProxy->ImplicitObjects_GameThread.Add(ImplicitSphere);
					else if (OutOptShapes) OutOptShapes->Add({ ImplicitSphere,true,true,InActor });
				}
				else
				{
					Chaos::FVec3 half_extents(0, 0, TaperedCapsule.Length / 2 * Scale[0]);
					auto ImplicitCylinder = MakeUnique<Chaos::FCylinder>(-half_extents, half_extents, TaperedCapsule.Radius * Scale[0]);
					if (PhysicsProxy) PhysicsProxy->ImplicitObjects_GameThread.Add(MoveTemp(ImplicitSphere));
					else if (OutOptShapes) OutOptShapes->Add({ ImplicitSphere,true,true,InActor });

					auto ImplicitSphereA = MakeUnique<Chaos::TSphere<FReal, 3>>(-half_extents, TaperedCapsule.Radius * Scale[0]);
					if (PhysicsProxy) PhysicsProxy->ImplicitObjects_GameThread.Add(MoveTemp(ImplicitSphereA));
					else if (OutOptShapes) OutOptShapes->Add({ ImplicitSphereA,true,true,InActor });

					auto ImplicitSphereB = MakeUnique<Chaos::TSphere<FReal, 3>>(half_extents, TaperedCapsule.Radius * Scale[0]);
					if (PhysicsProxy) PhysicsProxy->ImplicitObjects_GameThread.Add(MoveTemp(ImplicitSphereB));
					else if (OutOptShapes) OutOptShapes->Add({ ImplicitSphereB,true,true,InActor });
				}
			}
#endif
			if (!InParams.Geometry->ConvexElems.IsEmpty())
			{
				// Extract the scale from the transform - we have separate wrapper classes for scale versus translate/rotate 
				FVector NetScale = Scale * InParams.LocalTransform.GetScale3D();

				// If Scale is zero in any component, set minimum positive instead
				NetScale.X = FMath::Abs(NetScale.X) < UE_KINDA_SMALL_NUMBER ? UE_KINDA_SMALL_NUMBER : NetScale.X;
				NetScale.Y = FMath::Abs(NetScale.Y) < UE_KINDA_SMALL_NUMBER ? UE_KINDA_SMALL_NUMBER : NetScale.Y;
				NetScale.Z = FMath::Abs(NetScale.Z) < UE_KINDA_SMALL_NUMBER ? UE_KINDA_SMALL_NUMBER : NetScale.Z;

				const FVector NetScaleAbs = NetScale.GetAbs();

				const FTransform ConvexTransform = FTransform(InParams.LocalTransform.GetRotation(), Scale * InParams.LocalTransform.GetTranslation(), FVector(1, 1, 1));
				const bool bHasTranslationOrRotation = !ConvexTransform.GetTranslation().IsNearlyZero() || !ConvexTransform.GetRotation().IsIdentity();
				const bool bNoScale = NetScale == FVector(1);

				for (const FKConvexElem& CollisionBody : InParams.Geometry->ConvexElems)
				{
					if (const auto& ConvexImplicit = CollisionBody.GetChaosConvexMesh())
					{
						const FVector ScaledSize = (NetScaleAbs * CollisionBody.ElemBox.GetSize());	// Note: Scale can be negative
						const Chaos::FReal CollisionMargin = FMath::Min<Chaos::FReal>(ScaledSize.GetMin() * CollisionMarginFraction, CollisionMarginMax);

						// Wrap the convex in a scaled or instanced wrapper depending on scale value, and add a margin
						// NOTE: CollisionMargin is on the Instance/Scaled wrapper, not the inner convex (which is shared and should not have a margin).
						Chaos::FImplicitObjectPtr Implicit;
						if (bNoScale)
						{
							Implicit = MakeImplicitObjectPtr<Chaos::TImplicitObjectInstanced<Chaos::FConvex>>(ConvexImplicit, CollisionMargin);
						}
						else
						{
							Implicit = MakeImplicitObjectPtr<Chaos::TImplicitObjectScaled<Chaos::FConvex>>(ConvexImplicit, NetScale, CollisionMargin);
						}

						// Wrap the convex in a non-scaled transform if necessary (the scale is pulled out above)
						if (bHasTranslationOrRotation)
						{
							Implicit = MakeImplicitObjectPtr<Chaos::TImplicitObjectTransformed<Chaos::FReal, 3>>(MoveTemp(Implicit), ConvexTransform);
						}

						TUniquePtr<Chaos::FPerShapeData> NewShape = NewShapeHelper(Implicit, Shapes.Num(), (void*)CollisionBody.GetUserData(), CollisionBody.GetCollisionEnabled());
						Shapes.Emplace(MoveTemp(NewShape));
						Geoms.Emplace(MoveTemp(Implicit));
					}
				}
			}
		}

		if (bMakeComplexGeometry)
		{
			const FTransform MeshTransform = FTransform(InParams.LocalTransform.GetRotation(), Scale * InParams.LocalTransform.GetTranslation(), FVector(1, 1, 1));
			const bool bHasTranslationOrRotation = !MeshTransform.GetTranslation().IsNearlyZero() || !MeshTransform.GetRotation().IsIdentity();

			// Extract the scale from the transform - we have separate wrapper classes for scale versus translate/rotate 
			FVector NetScale = Scale * InParams.LocalTransform.GetScale3D();
			const bool bNoScale = FVector::PointsAreNear(NetScale, FVector(1), UE_KINDA_SMALL_NUMBER);

			for (auto& ChaosTriMesh : InParams.TriMeshGeometries)
			{
				Chaos::FImplicitObjectPtr Implicit;
				if (bNoScale)
				{
					Implicit = MakeImplicitObjectPtr<Chaos::TImplicitObjectInstanced<Chaos::FTriangleMeshImplicitObject>>(ChaosTriMesh);
				}
				else
				{
					Implicit = MakeImplicitObjectPtr<Chaos::TImplicitObjectScaled<Chaos::FTriangleMeshImplicitObject>>(ChaosTriMesh, NetScale);
				}

				// Wrap the mesh in a non-scaled transform if necessary (the scale is pulled out above)
				if (bHasTranslationOrRotation)
				{
					Implicit = MakeImplicitObjectPtr<Chaos::TImplicitObjectTransformed<Chaos::FReal, 3>>(MoveTemp(Implicit), MeshTransform);
				}

				ChaosTriMesh->SetCullsBackFaceRaycast(!InParams.bDoubleSided);

				TUniquePtr<Chaos::FPerShapeData> NewShape = NewShapeHelper(Implicit,Shapes.Num(), nullptr, ECollisionEnabled::QueryAndPhysics, true);
				Shapes.Emplace(MoveTemp(NewShape));
				Geoms.Emplace(MoveTemp(Implicit));
			}
		}
	}

	void CalculateMassPropertiesFromShapeCollection(Chaos::FMassProperties& OutProperties, const TArray<FPhysicsShapeHandle>& InShapes, float InDensityKGPerCM)
	{
		Chaos::CalculateMassPropertiesFromShapeCollection(
			OutProperties,
			InShapes.Num(),
			InDensityKGPerCM,
			TArray<bool>(),
			[&InShapes](int32 ShapeIndex) { return InShapes[ShapeIndex].Shape; }
		);
	}

	void CalculateMassPropertiesFromShapeCollection(Chaos::FMassProperties& OutProperties, const Chaos::FShapesArray& InShapes, const TArray<bool>& bContributesToMass, float InDensityKGPerCM)
	{
		Chaos::CalculateMassPropertiesFromShapeCollection(
			OutProperties,
			InShapes.Num(),
			InDensityKGPerCM,
			bContributesToMass,
			[&InShapes](int32 ShapeIndex) { return InShapes[ShapeIndex].Get(); }
		);
	}



}
