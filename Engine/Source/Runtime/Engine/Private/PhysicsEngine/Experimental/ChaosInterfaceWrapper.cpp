// Copyright Epic Games, Inc. All Rights Reserved.

#include "Physics/Experimental/ChaosInterfaceWrapper.h"
#include "Chaos/ChaosUserEntity.h"
#include "Chaos/PhysicsObject.h"
#include "Physics/Experimental/PhysScene_Chaos.h"
#include "PBDRigidsSolver.h"


namespace ChaosInterface
{
	FBodyInstance* GetUserData(const Chaos::FGeometryParticle& Actor)
	{
		void* UserData = Actor.UserData();
		if (UserData)
		{
			FBodyInstance* BodyInstance = FChaosUserData::Get<FBodyInstance>(UserData);
			if (!BodyInstance)
			{
				// Check if we appended a custom entity
				FChaosUserEntityAppend* ChaosUserEntityAppend = FChaosUserData::Get<FChaosUserEntityAppend>(UserData);
				if (ChaosUserEntityAppend)
				{
					BodyInstance = FChaosUserData::Get<FBodyInstance>(ChaosUserEntityAppend->ChaosUserData);
				}
			}
			return BodyInstance;
		}
		return nullptr;
	}

	UPhysicalMaterial* GetUserData(const Chaos::FChaosPhysicsMaterial& Material)
	{
		void* UserData = Material.UserData;
		return UserData ? FChaosUserData::Get<UPhysicalMaterial>(UserData) : nullptr;
	}

	UPrimitiveComponent* GetPrimitiveComponentFromUserData(const Chaos::FGeometryParticle& Actor)
	{
		void* UserData = Actor.UserData();
		if (UserData)
		{
			UPrimitiveComponent* PrimitiveComponent = FChaosUserData::Get<UPrimitiveComponent>(UserData);
			if (!PrimitiveComponent)
			{
				// Check if we appended a custom entity
				FChaosUserEntityAppend* ChaosUserEntityAppend = FChaosUserData::Get<FChaosUserEntityAppend>(UserData);
				if (ChaosUserEntityAppend)
				{
					PrimitiveComponent = FChaosUserData::Get<UPrimitiveComponent>(ChaosUserEntityAppend->ChaosUserData);
				}
			}
			return PrimitiveComponent;
		}
		return nullptr;
	}

	Chaos::FPhysicsObject* GetPhysicsObjectFromUserData(const Chaos::FGeometryParticle& Actor)
	{
		void* UserData = Actor.UserData();
		if (UserData)
		{
			Chaos::FPhysicsObject* PhysicsObject = FChaosUserData::Get<Chaos::FPhysicsObject>(UserData);
			if (!PhysicsObject)
			{
				// Check if we appended a custom entity
				FChaosUserEntityAppend* ChaosUserEntityAppend = FChaosUserData::Get<FChaosUserEntityAppend>(UserData);
				if (ChaosUserEntityAppend)
				{
					PhysicsObject = FChaosUserData::Get<Chaos::FPhysicsObject>(ChaosUserEntityAppend->ChaosUserData);
				}
			}
			return PhysicsObject;
		}
		return nullptr;
	}

	bool HasValidUserData(const Chaos::FGeometryParticle& Actor)
	{
		if (FChaosUserData* UserData = static_cast<FChaosUserData*>(Actor.UserData()))
		{
			return !UserData->Is<EChaosUserDataType::Invalid>();
		}

		return false;
	}

	FScopedSceneReadLock::FScopedSceneReadLock(FPhysScene_Chaos& SceneIn)
		: Solver(SceneIn.GetSolver())
	{
		if(Solver)
		{
			Solver->GetExternalDataLock_External().ReadLock();
		}
	}

	FScopedSceneReadLock::~FScopedSceneReadLock()
	{
		if(Solver)
		{
			Solver->GetExternalDataLock_External().ReadUnlock();
		}
	}
}
