// Copyright Epic Games, Inc. All Rights Reserved.

#include "Physics/Experimental/ChaosScopedSceneLock.h"

#include "Chaos/PBDJointConstraintData.h"
#include "Components/SkeletalMeshComponent.h"
#include "Physics/Experimental/PhysScene_Chaos.h"
#include "PhysicsEngine/ConstraintInstance.h"
#include "PhysicsEngine/PhysicsObjectExternalInterface.h"
#include "PhysicsProxy/SingleParticlePhysicsProxy.h"
#include "PBDRigidsSolver.h"

namespace
{
	Chaos::FPBDRigidsSolver* GetSolverForActor(const FPhysicsActorHandle& InActorHandle)
	{
		return InActorHandle->GetSolver<Chaos::FPBDRigidsSolver>();
	}
}

FScopedSceneLock_Chaos::FScopedSceneLock_Chaos(const FPhysicsActorHandle& InActorHandle, EPhysicsInterfaceScopedLockType InLockType)
	: LockType(InLockType)
{
	Solver = GetSolverForActor(InActorHandle);
	LockScene();
}

FScopedSceneLock_Chaos::FScopedSceneLock_Chaos(const FPhysicsActorHandle& InActorHandleA, const FPhysicsActorHandle& InActorHandleB, EPhysicsInterfaceScopedLockType InLockType)
	: LockType(InLockType)
{
	FChaosScene* SceneA = GetSceneForActor(InActorHandleA);
	FChaosScene* SceneB = GetSceneForActor(InActorHandleB);
	FChaosScene* Scene = nullptr;

	if (SceneA == SceneB)
	{
		Scene = SceneA;
	}
	else if (!SceneA || !SceneB)
	{
		Scene = SceneA ? SceneA : SceneB;
	}
	else
	{
		UE_LOGF(LogPhysics, Warning, "Attempted to aquire a physics scene lock for two paired actors that were not in the same scene. Skipping lock");
	}

	Solver = Scene ? Scene->GetSolver() : nullptr;
	LockScene();
}

FScopedSceneLock_Chaos::FScopedSceneLock_Chaos(FPhysicsActorHandle const* InActorHandle, EPhysicsInterfaceScopedLockType InLockType)
	: LockType(InLockType)
{
	FChaosScene* Scene = GetSceneForActor(*InActorHandle);
	Solver = Scene ? Scene->GetSolver() : nullptr;
	LockScene();
}

FScopedSceneLock_Chaos::FScopedSceneLock_Chaos(FPhysicsActorHandle const* InActorHandleA, FPhysicsActorHandle const* InActorHandleB, EPhysicsInterfaceScopedLockType InLockType)
	: LockType(InLockType)
{
	FChaosScene* SceneA = (InActorHandleA != nullptr) ? GetSceneForActor(*InActorHandleA) : nullptr;
	FChaosScene* SceneB = (InActorHandleB != nullptr) ? GetSceneForActor(*InActorHandleB) : nullptr;
	FChaosScene* Scene = nullptr;

	if (SceneA == SceneB)
	{
		Scene = SceneA;
	}
	else if (!SceneA || !SceneB)
	{
		Scene = SceneA ? SceneA : SceneB;
	}
	else
	{
		UE_LOGF(LogPhysics, Warning, "Attempted to aquire a physics scene lock for two paired actors that were not in the same scene. Skipping lock");
	}

	Solver = Scene ? Scene->GetSolver() : nullptr;
	LockScene();
}

FScopedSceneLock_Chaos::FScopedSceneLock_Chaos(FPhysicsConstraintHandle const* InConstraintHandle, EPhysicsInterfaceScopedLockType InLockType)
	: Solver(nullptr),
	LockType(InLockType)
{
	LockSceneForConstraint(InConstraintHandle);
}

FScopedSceneLock_Chaos::FScopedSceneLock_Chaos(USkeletalMeshComponent* InSkelMeshComp, EPhysicsInterfaceScopedLockType InLockType)
	: LockType(InLockType)
{
	Solver = nullptr;

	if (InSkelMeshComp)
	{
		for (FBodyInstance* BI : InSkelMeshComp->Bodies)
		{
			FChaosScene* Scene = GetSceneForActor(BI->GetPhysicsActor());
			if (Scene)
			{
				Solver = Scene->GetSolver();
				break;
			}
		}
	}

	LockScene();
}

FScopedSceneLock_Chaos::FScopedSceneLock_Chaos(Chaos::FPhysicsObjectHandle InObjectA, Chaos::FPhysicsObjectHandle InObjectB, EPhysicsInterfaceScopedLockType InLockType)
	: LockType(InLockType)
{
	FChaosScene* SceneA = static_cast<FChaosScene*>(FPhysicsObjectExternalInterface::GetScene({ &InObjectA, 1 }));
	FChaosScene* SceneB = static_cast<FChaosScene*>(FPhysicsObjectExternalInterface::GetScene({ &InObjectB, 1 }));
	FChaosScene* Scene = nullptr;

	if (SceneA == SceneB)
	{
		Scene = SceneA;
	}
	else if (!SceneA || !SceneB)
	{
		Scene = SceneA ? SceneA : SceneB;
	}
	else
	{
		UE_LOGF(LogPhysics, Warning, "Attempted to aquire a physics scene lock for two paired physics objects that were not in the same scene. Skipping lock");
	}

	Solver = Scene ? Scene->GetSolver() : nullptr;
	LockScene();
}

FScopedSceneLock_Chaos::FScopedSceneLock_Chaos(FChaosScene* InScene, EPhysicsInterfaceScopedLockType InLockType)
	: Solver(InScene ? InScene->GetSolver() : nullptr),
	LockType(InLockType)
{
	LockScene();
}

FScopedSceneLock_Chaos::FScopedSceneLock_Chaos(FScopedSceneLock_Chaos&& Other)
{
	*this = MoveTemp(Other);
}

FScopedSceneLock_Chaos& FScopedSceneLock_Chaos::operator=(FScopedSceneLock_Chaos&& Other)
{
	Solver = Other.Solver;
	LockType = Other.LockType;
	ThreadContext = Other.ThreadContext;
	bHasLock = Other.bHasLock;

	Other.bHasLock = false;
	Other.Solver = nullptr;
	return *this;
}

FScopedSceneLock_Chaos::~FScopedSceneLock_Chaos()
{
	Release();
}

void FScopedSceneLock_Chaos::Release()
{
	if (bHasLock)
	{
		UnlockScene();
	}
}

void FScopedSceneLock_Chaos::LockScene()
{
	Chaos::FPhysSceneLock* SceneLock = GetSolverLock(Solver);
	if (!SceneLock)
	{
		return;
	}

	switch (LockType)
	{
	case EPhysicsInterfaceScopedLockType::Read:
		SceneLock->ReadLock();
		break;
	case EPhysicsInterfaceScopedLockType::Write:
		{
#if UE_WITH_REMOTE_OBJECT_HANDLE
			if (TransactionMode == EPhysicsInterfaceScopedTransactionMode::MultiServer && !Chaos::Private::FDstmScopedWriteLockInvalidationIgnore::WriteLocksInvalidationAreIgnored())
			{
				UE::RemoteExecutor::TransactionRequiresMultiServerCommit(TEXT("Physics Write Lock"));
			}
#endif
			SceneLock->WriteLock();
			break;
		}
	}

	bHasLock = true;
}

void FScopedSceneLock_Chaos::UnlockScene()
{
	Chaos::FPhysSceneLock* SceneLock = GetSolverLock(Solver);
	if (!SceneLock)
	{
		return;
	}

	switch (LockType)
	{
	case EPhysicsInterfaceScopedLockType::Read:
		SceneLock->ReadUnlock();
		break;
	case EPhysicsInterfaceScopedLockType::Write:
		SceneLock->WriteUnlock();
		break;
	}

	bHasLock = false;
}

FChaosScene* FScopedSceneLock_Chaos::GetSceneForActor(const FPhysicsActorHandle& InActorHandle)
{
	if (InActorHandle)
	{
		return static_cast<FPhysScene*>(FChaosEngineInterface::GetCurrentScene(InActorHandle));
	}

	return nullptr;
}

FChaosScene* FScopedSceneLock_Chaos::GetSceneForActor(FPhysicsConstraintHandle const* InConstraintHandle)
{
	if (InConstraintHandle && InConstraintHandle->IsValid() && InConstraintHandle->Constraint->IsType(Chaos::EConstraintType::JointConstraintType))
	{
		Chaos::FJointConstraint* Constraint = static_cast<Chaos::FJointConstraint*>(InConstraintHandle->Constraint);

		FConstraintInstanceBase* ConstraintInstance = (Constraint) ? FPhysicsUserData_Chaos::Get<FConstraintInstanceBase>(Constraint->GetUserData()) : nullptr;
		if (ConstraintInstance)
		{
			return ConstraintInstance->GetPhysicsScene();
		}
	}

	return nullptr;
}

Chaos::FPhysSceneLock* FScopedSceneLock_Chaos::GetSolverLock(Chaos::FPBDRigidsSolver* InSolver)
{
	if (!InSolver)
	{
		return nullptr;
	}

#if UE_WITH_REMOTE_OBJECT_HANDLE
	switch (ThreadContext)
	{
		case EPhysicsInterfaceScopedThreadContext::Internal:
			return &InSolver->GetInternalDataLock();
		case EPhysicsInterfaceScopedThreadContext::External:
			return &InSolver->GetExternalDataLock_External();
		default:
			{
				UE_LOGF(LogChaos, Fatal, "Unsupported thread context used.")
				return nullptr;
			}
	}
#else
	return &InSolver->GetExternalDataLock_External();
#endif
}

void FScopedSceneLock_Chaos::LockSceneForConstraint(FPhysicsConstraintHandle const* InConstraintHandle)
{
	if (InConstraintHandle)
	{
		FChaosScene* Scene = GetSceneForActor(InConstraintHandle);
		Solver = Scene ? Scene->GetSolver() : nullptr;
	}
#if CHAOS_CHECKED
	if (!Solver)
	{
		UE_LOGF(LogPhysics, Warning, "Failed to find Scene for constraint. Skipping lock");
	}
#endif
	LockScene();
}

FScopedSceneLockWithContext_Chaos::FScopedSceneLockWithContext_Chaos(const FPhysicsActorHandle& InActorHandle, EPhysicsInterfaceScopedLockType InLockType, EPhysicsInterfaceScopedThreadContext InThreadContext, EPhysicsInterfaceScopedTransactionMode InTransactionMode)
	: FScopedSceneLock_Chaos(InLockType)
{
	ThreadContext = InThreadContext;
#if UE_WITH_REMOTE_OBJECT_HANDLE
	TransactionMode = InTransactionMode;
#endif
	Solver = GetSolverForActor(InActorHandle);
	LockScene();
}

FScopedSceneLockWithContext_Chaos::FScopedSceneLockWithContext_Chaos(FPhysicsConstraintHandle const* InConstraintHandle, EPhysicsInterfaceScopedLockType InLockType, EPhysicsInterfaceScopedThreadContext InThreadContext, EPhysicsInterfaceScopedTransactionMode InTransactionMode)
	: FScopedSceneLock_Chaos(InLockType)
{
	ThreadContext = InThreadContext;
#if UE_WITH_REMOTE_OBJECT_HANDLE
	TransactionMode = InTransactionMode;
#endif
	LockSceneForConstraint(InConstraintHandle);
}

#if UE_WITH_REMOTE_OBJECT_HANDLE
namespace Chaos::Private
{
	thread_local bool bIgnoreWriteLockInvalidation = false;

	FDstmScopedWriteLockInvalidationIgnore::FDstmScopedWriteLockInvalidationIgnore(bool bIgnore)
	{
		bOldState = bIgnoreWriteLockInvalidation;
		bIgnoreWriteLockInvalidation = bIgnore;
	}

	FDstmScopedWriteLockInvalidationIgnore::~FDstmScopedWriteLockInvalidationIgnore()
	{
		bIgnoreWriteLockInvalidation = bOldState;
	}

	bool FDstmScopedWriteLockInvalidationIgnore::WriteLocksInvalidationAreIgnored()
	{
		return bIgnoreWriteLockInvalidation || UE::RemoteObject::Handle::IsTransactionallyPostMigratingObjects();
	}
} // namespace Chaos::Private

#endif // UE_WITH_REMOTE_OBJECT_HANDLE
