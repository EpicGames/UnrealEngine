// Copyright Epic Games, Inc. All Rights Reserved.

#include "Physics/Experimental/PhysInterface_Chaos.h"
#include "Chaos/CollisionConvexMesh.h"
#include "Physics/Experimental/PhysScene_Chaos.h"
#include "CollisionShape.h"
#include "Physics/Experimental/ChaosInterfaceUtils.h"
#include "EngineLogs.h"
#include "Physics/Experimental/ChaosScopedSceneLock.h"
#include "Physics/Experimental/ChaosInterfaceWrapper.h"
#include "Physics/PhysicsInterfaceTypes.h"

#include "Chaos/GeometryQueries.h"
#include "Chaos/PBDJointConstraintData.h"
#include "Chaos/ChaosConstraintSettings.h"

#include "Collision/CollisionConversions.h"
#include "Components/PrimitiveComponent.h"
#include "Materials/MaterialInterface.h"
#include "PBDRigidsSolver.h"
#include "Engine/World.h"
#include "PhysicsEngine/BodySetup.h"
#include "Physics/Experimental/ChaosScopedSceneLock.h"
#include "Physics/PhysicsFiltering.h"
#include "PhysicalMaterials/PhysicalMaterialMask.h"
#include "PhysicalMaterials/PhysicalMaterial.h"
#include "PhysicsEngine/ConstraintDrives.h"
#include "PhysicsEngine/PhysicsConstraintComponent.h"
#include "PhysicsProxy/SingleParticlePhysicsProxy.h"
#include "PhysicsEngine/ConstraintTypes.h"

DEFINE_STAT(STAT_TotalPhysicsTime);
DEFINE_STAT(STAT_NumCloths);
DEFINE_STAT(STAT_NumClothVerts);

DECLARE_CYCLE_STAT(TEXT("Start Physics Time (sync)"), STAT_PhysicsKickOffDynamicsTime, STATGROUP_Physics);
DECLARE_CYCLE_STAT(TEXT("Fetch Results Time (sync)"), STAT_PhysicsFetchDynamicsTime, STATGROUP_Physics);

DECLARE_CYCLE_STAT(TEXT("Start Physics Time (async)"), STAT_PhysicsKickOffDynamicsTime_Async, STATGROUP_Physics);
DECLARE_CYCLE_STAT(TEXT("Fetch Results Time (async)"), STAT_PhysicsFetchDynamicsTime_Async, STATGROUP_Physics);

DECLARE_CYCLE_STAT(TEXT("Update Kinematics On Deferred SkelMeshes"), STAT_UpdateKinematicsOnDeferredSkelMeshes, STATGROUP_Physics);

DECLARE_CYCLE_STAT(TEXT("Phys Events Time"), STAT_PhysicsEventTime, STATGROUP_Physics);
DECLARE_CYCLE_STAT(TEXT("SyncComponentsToBodies (sync)"), STAT_SyncComponentsToBodies, STATGROUP_Physics);
DECLARE_CYCLE_STAT(TEXT("SyncComponentsToBodies (async)"), STAT_SyncComponentsToBodies_Async, STATGROUP_Physics);
DECLARE_CYCLE_STAT(TEXT("Query PhysicalMaterialMask Hit"), STAT_QueryPhysicalMaterialMaskHit, STATGROUP_Physics);

DECLARE_DWORD_COUNTER_STAT(TEXT("Broadphase Adds"), STAT_NumBroadphaseAdds, STATGROUP_Physics);
DECLARE_DWORD_COUNTER_STAT(TEXT("Broadphase Removes"), STAT_NumBroadphaseRemoves, STATGROUP_Physics);
DECLARE_DWORD_COUNTER_STAT(TEXT("Active Constraints"), STAT_NumActiveConstraints, STATGROUP_Physics);
DECLARE_DWORD_COUNTER_STAT(TEXT("Active Simulated Bodies"), STAT_NumActiveSimulatedBodies, STATGROUP_Physics);
DECLARE_DWORD_COUNTER_STAT(TEXT("Active Kinematic Bodies"), STAT_NumActiveKinematicBodies, STATGROUP_Physics);
DECLARE_DWORD_COUNTER_STAT(TEXT("Mobile Bodies"), STAT_NumMobileBodies, STATGROUP_Physics);
DECLARE_DWORD_COUNTER_STAT(TEXT("Static Bodies"), STAT_NumStaticBodies, STATGROUP_Physics);
DECLARE_DWORD_COUNTER_STAT(TEXT("Shapes"), STAT_NumShapes, STATGROUP_Physics);

DECLARE_DWORD_COUNTER_STAT(TEXT("(ASync) Broadphase Adds"), STAT_NumBroadphaseAddsAsync, STATGROUP_Physics);
DECLARE_DWORD_COUNTER_STAT(TEXT("(ASync) Broadphase Removes"), STAT_NumBroadphaseRemovesAsync, STATGROUP_Physics);
DECLARE_DWORD_COUNTER_STAT(TEXT("(ASync) Active Constraints"), STAT_NumActiveConstraintsAsync, STATGROUP_Physics);
DECLARE_DWORD_COUNTER_STAT(TEXT("(ASync) Active Simulated Bodies"), STAT_NumActiveSimulatedBodiesAsync, STATGROUP_Physics);
DECLARE_DWORD_COUNTER_STAT(TEXT("(ASync) Active Kinematic Bodies"), STAT_NumActiveKinematicBodiesAsync, STATGROUP_Physics);
DECLARE_DWORD_COUNTER_STAT(TEXT("(ASync) Mobile Bodies"), STAT_NumMobileBodiesAsync, STATGROUP_Physics);
DECLARE_DWORD_COUNTER_STAT(TEXT("(ASync) Static Bodies"), STAT_NumStaticBodiesAsync, STATGROUP_Physics);
DECLARE_DWORD_COUNTER_STAT(TEXT("(ASync) Shapes"), STAT_NumShapesAsync, STATGROUP_Physics);

ECollisionShapeType GetGeometryType(const Chaos::FPerShapeData& Shape)
{
	return ChaosInterface::GetType(*Shape.GetGeometry());
}

Chaos::FChaosPhysicsMaterial* GetMaterialFromInternalFaceIndex(const FPhysicsShape& Shape, const FPhysicsActor& Actor, uint32 InternalFaceIndex)
{
	const auto& Materials = Shape.GetMaterials();
	if(Materials.Num() > 0 && Actor.GetProxy())
	{
		Chaos::FPBDRigidsSolver* Solver = Actor.GetProxy()->GetSolver<Chaos::FPBDRigidsSolver>();

		if(ensure(Solver))
		{
			if(Materials.Num() == 1)
			{
				return Solver->GetQueryMaterials_External().Get(Materials[0].InnerHandle);
			}

			uint8 Index = Shape.GetGeometry()->GetMaterialIndex(InternalFaceIndex);

			if(Materials.IsValidIndex(Index))
			{
				return Solver->GetQueryMaterials_External().Get(Materials[Index].InnerHandle);
			}
		}
	}

	return nullptr;
}

Chaos::FChaosPhysicsMaterial* GetMaterialFromInternalFaceIndexAndHitLocation(const FPhysicsShape& Shape, const FPhysicsActor& Actor, uint32 InternalFaceIndex, const FVector& HitLocation)
{
	using namespace ChaosInterface;

	{
		SCOPE_CYCLE_COUNTER(STAT_QueryPhysicalMaterialMaskHit);

		if (Shape.GetMaterials().Num() > 0 && Actor.GetProxy())
		{
			Chaos::FPBDRigidsSolver* Solver = Actor.GetProxy()->GetSolver<Chaos::FPBDRigidsSolver>();

			if (ensure(Solver))
			{
				if (Shape.GetMaterialMasks().Num() > 0)
				{
					UBodySetup* BodySetup = nullptr;

					if (const FBodyInstance* BodyInst = GetUserData(Actor))
					{
						BodyInst = FPhysicsInterface::ShapeToOriginalBodyInstance(BodyInst, &Shape);
						BodySetup = BodyInst->GetBodySetup();	//this data should be immutable at runtime so ok to check from worker thread.
						ECollisionShapeType GeomType = GetGeometryType(Shape);

						if (BodySetup && BodySetup->bSupportUVsAndFaceRemap && GetGeometryType(Shape) == ECollisionShapeType::Trimesh)
						{
							FVector Scale(1.0f, 1.0f, 1.0f);
							const Chaos::FImplicitObject* Geometry = Shape.GetGeometry();
							if (const Chaos::TImplicitObjectScaled<Chaos::FTriangleMeshImplicitObject>* ScaledTrimesh = Chaos::TImplicitObjectScaled<Chaos::FTriangleMeshImplicitObject>::AsScaled(*Geometry))
							{
								Scale = ScaledTrimesh->GetScale();
							}

							// Convert hit location to local
							Chaos::FRigidTransform3 ActorToWorld(Actor.X(), Actor.R(), Scale);
							const FVector LocalHitPos = ActorToWorld.InverseTransformPosition(HitLocation);

							uint8 Index = Shape.GetGeometry()->GetMaterialIndex(InternalFaceIndex);
							if (Shape.GetMaterialMasks().IsValidIndex(Index))
							{
								Chaos::FChaosPhysicsMaterialMask* Mask = nullptr;
								{
									Mask = Solver->GetQueryMaterialMasks_External().Get(Shape.GetMaterialMasks()[Index].InnerHandle);
								}

								if (Mask && InternalFaceIndex < (uint32)BodySetup->FaceRemap.Num())
								{
									int32 RemappedFaceIndex = BodySetup->FaceRemap[InternalFaceIndex];
									FVector2D UV;


									if (BodySetup->CalcUVAtLocation(LocalHitPos, RemappedFaceIndex, Mask->UVChannelIndex, UV))
									{
										uint32 MapIdx = UPhysicalMaterialMask::GetPhysMatIndex(Mask->MaskData, Mask->SizeX, Mask->SizeY, Mask->AddressX, Mask->AddressY, UV.X, UV.Y);
										uint32 AdjustedMapIdx = Index * EPhysicalMaterialMaskColor::MAX + MapIdx;
										if (Shape.GetMaterialMaskMaps().IsValidIndex(AdjustedMapIdx))
										{
											uint32 MaterialIdx = Shape.GetMaterialMaskMaps()[AdjustedMapIdx];
											if (Shape.GetMaterialMaskMapMaterials().IsValidIndex(MaterialIdx))
											{
												return Solver->GetQueryMaterials_External().Get(Shape.GetMaterialMaskMapMaterials()[MaterialIdx].InnerHandle);
											}
										}
									}
								}
							}
						}
					}
				}
			}
		}
	}

	return GetMaterialFromInternalFaceIndex(Shape, Actor, InternalFaceIndex);
}

FPhysInterface_Chaos::FPhysInterface_Chaos(const AWorldSettings* Settings) 
{

}

FPhysInterface_Chaos::~FPhysInterface_Chaos()
{
}

FPhysicsMaterialMaskHandle FPhysInterface_Chaos::CreateMaterialMask(const UPhysicalMaterialMask* InMaterialMask)
{
	Chaos::FMaterialMaskHandle NewHandle = Chaos::FPhysicalMaterialManager::Get().CreateMask();
	FPhysInterface_Chaos::UpdateMaterialMask(NewHandle, InMaterialMask);
	return NewHandle;
}

void FPhysInterface_Chaos::UpdateMaterialMask(FPhysicsMaterialMaskHandle& InHandle, const UPhysicalMaterialMask* InMaterialMask)
{
	if (Chaos::FChaosPhysicsMaterialMask* MaterialMask = InHandle.Get())
	{
		InMaterialMask->GenerateMaskData(MaterialMask->MaskData, MaterialMask->SizeX, MaterialMask->SizeY);
		MaterialMask->UVChannelIndex = InMaterialMask->UVChannelIndex;
		MaterialMask->AddressX = static_cast<int32>(InMaterialMask->AddressX);
		MaterialMask->AddressY = static_cast<int32>(InMaterialMask->AddressY);
	}

	Chaos::FPhysicalMaterialManager::Get().UpdateMaterialMask(InHandle);
}

FPhysScene* FPhysInterface_Chaos::GetCurrentSceneForComponent(USceneComponent* InComponent)
{
	if (!InComponent)
	{
		return nullptr;
	}

	if (UPrimitiveComponent* PrimComp = Cast<UPrimitiveComponent>(InComponent))
	{
		FBodyInstance* BodyInstance = PrimComp->GetBodyInstance();
		if (BodyInstance && BodyInstance->GetPhysicsActorHandle())
		{
			return static_cast<FPhysScene*>(GetCurrentScene(BodyInstance->GetPhysicsActorHandle()));
		}
	}

	if (UWorld* ComponentWorld = InComponent->GetWorld())
	{
		return ComponentWorld->GetPhysicsScene();
	}

	return nullptr;
}

bool FPhysInterface_Chaos::IsInScene(const FPhysicsActorHandle& InActorReference)
{
	return (GetCurrentScene(InActorReference) != nullptr);
}

void FPhysInterface_Chaos::FlushScene(FPhysScene* InScene)
{
	InScene->Flush();
}

Chaos::EJointMotionType ConvertMotionType(ELinearConstraintMotion InEngineType)
{
	if (InEngineType == ELinearConstraintMotion::LCM_Free)
		return Chaos::EJointMotionType::Free;
	else if (InEngineType == ELinearConstraintMotion::LCM_Limited)
		return Chaos::EJointMotionType::Limited;
	else if (InEngineType == ELinearConstraintMotion::LCM_Locked)
		return Chaos::EJointMotionType::Locked;
	else
		ensure(false);
	return Chaos::EJointMotionType::Locked;
};

void FPhysInterface_Chaos::SetLinearMotionLimitType_AssumesLocked(const FPhysicsConstraintHandle& InConstraintRef, PhysicsInterfaceTypes::ELimitAxis InAxis, ELinearConstraintMotion InMotion)
{
	if (InConstraintRef.IsValid() && InConstraintRef.Constraint->IsType(Chaos::EConstraintType::JointConstraintType))
	{
		if (Chaos::FJointConstraint* Constraint = static_cast<Chaos::FJointConstraint*>(InConstraintRef.Constraint))
		{
			switch (InAxis)
			{
			case PhysicsInterfaceTypes::ELimitAxis::X:
				Constraint->SetLinearMotionTypesX(ConvertMotionType(InMotion));
				break;

			case PhysicsInterfaceTypes::ELimitAxis::Y:
				Constraint->SetLinearMotionTypesY(ConvertMotionType(InMotion));
				break;

			case PhysicsInterfaceTypes::ELimitAxis::Z:
				Constraint->SetLinearMotionTypesZ(ConvertMotionType(InMotion));
				break;
			default:
				ensure(false);
			}
		}
	}
}

Chaos::EJointMotionType ConvertMotionType(EAngularConstraintMotion InEngineType)
{
	if (InEngineType == EAngularConstraintMotion::ACM_Free)
		return Chaos::EJointMotionType::Free;
	else if (InEngineType == EAngularConstraintMotion::ACM_Limited)
		return Chaos::EJointMotionType::Limited;
	else if (InEngineType == EAngularConstraintMotion::ACM_Locked)
		return Chaos::EJointMotionType::Locked;
	else
		ensure(false);
	return Chaos::EJointMotionType::Locked;
};


void FPhysInterface_Chaos::SetAngularMotionLimitType_AssumesLocked(const FPhysicsConstraintHandle& InConstraintRef, PhysicsInterfaceTypes::ELimitAxis InAxis, EAngularConstraintMotion InMotion)
{
	// Twist is X component, Swing1 is Z component, and Swing2 is Y component in Chaos (see EJointAngularConstraintIndex)
	static_assert(((int32)Chaos::EJointAngularConstraintIndex::Twist == 0), "EJointAngularConstraintIndex has changed");
	static_assert(((int32)Chaos::EJointAngularConstraintIndex::Swing1 == 2), "EJointAngularConstraintIndex has changed");
	static_assert(((int32)Chaos::EJointAngularConstraintIndex::Swing2 == 1), "EJointAngularConstraintIndex has changed");

	if (InConstraintRef.IsValid() && InConstraintRef.Constraint->IsType(Chaos::EConstraintType::JointConstraintType))
	{
		if (Chaos::FJointConstraint* Constraint = static_cast<Chaos::FJointConstraint*>(InConstraintRef.Constraint))
		{
			switch (InAxis)
			{
			case PhysicsInterfaceTypes::ELimitAxis::Twist:
				Constraint->SetAngularMotionTypesX(ConvertMotionType(InMotion));
				break;

			case PhysicsInterfaceTypes::ELimitAxis::Swing1:
				Constraint->SetAngularMotionTypesZ(ConvertMotionType(InMotion));
				break;

			case PhysicsInterfaceTypes::ELimitAxis::Swing2:
				Constraint->SetAngularMotionTypesY(ConvertMotionType(InMotion));
				break;
			default:
				ensure(false);
			}
		}
	}
}

void FPhysInterface_Chaos::UpdateLinearLimitParams_AssumesLocked(const FPhysicsConstraintHandle& InConstraintRef, float InLimit, float InAverageMass, const FLinearConstraint& InParams)
{
	if (InConstraintRef.IsValid() && InConstraintRef.Constraint->IsType(Chaos::EConstraintType::JointConstraintType))
	{
		if (Chaos::FJointConstraint* Constraint = static_cast<Chaos::FJointConstraint*>(InConstraintRef.Constraint))
		{
			Constraint->SetLinearLimit(InLimit); 

			Constraint->SetSoftLinearLimitsEnabled(InParams.bSoftConstraint);
			Constraint->SetSoftLinearStiffness(Chaos::ConstraintSettings::SoftLinearStiffnessScale() * InParams.Stiffness);
			Constraint->SetSoftLinearDamping(Chaos::ConstraintSettings::SoftLinearDampingScale() * InParams.Damping);
			Constraint->SetLinearContactDistance(InParams.ContactDistance);
			Constraint->SetLinearRestitution(InParams.Restitution);
			//Constraint->SetAngularSoftForceMode( InParams.NOT_QUITE_SURE ); // @todo(chaos) 
		}
	}
}

void FPhysInterface_Chaos::UpdateConeLimitParams_AssumesLocked(const FPhysicsConstraintHandle& InConstraintRef, float InAverageMass, const FConeConstraint& InParams)
{
	if (InConstraintRef.IsValid() && InConstraintRef.Constraint->IsType(Chaos::EConstraintType::JointConstraintType))
	{
		if (Chaos::FJointConstraint* Constraint = static_cast<Chaos::FJointConstraint*>(InConstraintRef.Constraint))
		{
			Chaos::FVec3 Limit = Constraint->GetAngularLimits();
			Limit[(int32)Chaos::EJointAngularConstraintIndex::Swing1] = FMath::DegreesToRadians(InParams.Swing1LimitDegrees);
			Limit[(int32)Chaos::EJointAngularConstraintIndex::Swing2] = FMath::DegreesToRadians(InParams.Swing2LimitDegrees);
			Constraint->SetAngularLimits(Limit);

			Constraint->SetSoftSwingLimitsEnabled(InParams.bSoftConstraint);
			Constraint->SetSoftSwingStiffness(Chaos::ConstraintSettings::SoftAngularStiffnessScale() * InParams.Stiffness);
			Constraint->SetSoftSwingDamping(Chaos::ConstraintSettings::SoftAngularDampingScale() * InParams.Damping);
			Constraint->SetSwingContactDistance(InParams.ContactDistance);
			Constraint->SetSwingRestitution(InParams.Restitution);
			//Constraint->SetAngularSoftForceMode( InParams.NOT_QUITE_SURE ); // @todo(chaos) 
		}
	}
}

void FPhysInterface_Chaos::UpdateTwistLimitParams_AssumesLocked(const FPhysicsConstraintHandle& InConstraintRef, float InAverageMass, const FTwistConstraint& InParams)
{
	if (InConstraintRef.IsValid() && InConstraintRef.Constraint->IsType(Chaos::EConstraintType::JointConstraintType))
	{
		if (Chaos::FJointConstraint* Constraint = static_cast<Chaos::FJointConstraint*>(InConstraintRef.Constraint))
		{
			Chaos::FVec3 Limit = Constraint->GetAngularLimits();
			Limit[(int32)Chaos::EJointAngularConstraintIndex::Twist] = FMath::DegreesToRadians(InParams.TwistLimitDegrees);
			Constraint->SetAngularLimits(Limit);

			Constraint->SetSoftTwistLimitsEnabled(InParams.bSoftConstraint);
			Constraint->SetSoftTwistStiffness(Chaos::ConstraintSettings::SoftAngularStiffnessScale() * InParams.Stiffness);
			Constraint->SetSoftTwistDamping(Chaos::ConstraintSettings::SoftAngularDampingScale() * InParams.Damping);
			Constraint->SetTwistContactDistance(InParams.ContactDistance);
			Constraint->SetTwistRestitution(InParams.Restitution);
			//Constraint->SetAngularSoftForceMode( InParams.NOT_QUITE_SURE ); // @todo(chaos) 

		}
	}
}

void FPhysInterface_Chaos::UpdateLinearDrive_AssumesLocked(const FPhysicsConstraintHandle& InConstraintRef, const FLinearDriveConstraint& InDriveParams, bool InInitialize)
{
	if (InConstraintRef.IsValid() && InConstraintRef.Constraint->IsType(Chaos::EConstraintType::JointConstraintType))
	{
		if (Chaos::FJointConstraint* Constraint = static_cast<Chaos::FJointConstraint*>(InConstraintRef.Constraint))
		{
			Constraint->SetLinearPositionDriveXEnabled(false);
			Constraint->SetLinearPositionDriveYEnabled(false);
			Constraint->SetLinearPositionDriveZEnabled(false);

			Constraint->SetLinearVelocityDriveXEnabled(false);
			Constraint->SetLinearVelocityDriveYEnabled(false);
			Constraint->SetLinearVelocityDriveZEnabled(false);

			bool bPositionDriveEnabled = InDriveParams.IsPositionDriveEnabled();
			if (bPositionDriveEnabled)
			{
				Constraint->SetLinearPositionDriveXEnabled(InDriveParams.XDrive.bEnablePositionDrive);
				Constraint->SetLinearPositionDriveYEnabled(InDriveParams.YDrive.bEnablePositionDrive);
				Constraint->SetLinearPositionDriveZEnabled(InDriveParams.ZDrive.bEnablePositionDrive);
				if (InInitialize || FMath::IsNearlyEqual(Constraint->GetLinearPlasticityLimit(), (Chaos::FReal)FLT_MAX))
				{
					Constraint->SetLinearDrivePositionTarget(InDriveParams.PositionTarget);
				}
			}

			bool bVelocityDriveEnabled = InDriveParams.IsVelocityDriveEnabled();
			if (bVelocityDriveEnabled)
			{
				Constraint->SetLinearVelocityDriveXEnabled(InDriveParams.XDrive.bEnableVelocityDrive);
				Constraint->SetLinearVelocityDriveYEnabled(InDriveParams.YDrive.bEnableVelocityDrive);
				Constraint->SetLinearVelocityDriveZEnabled(InDriveParams.ZDrive.bEnableVelocityDrive);
				Constraint->SetLinearDriveVelocityTarget(InDriveParams.VelocityTarget);
			}

			Constraint->SetLinearDriveForceMode(InDriveParams.bAccelerationMode ? Chaos::EJointForceMode::Acceleration : Chaos::EJointForceMode::Force);
			Constraint->SetLinearDriveStiffness(Chaos::ConstraintSettings::LinearDriveStiffnessScale() * Chaos::FVec3(InDriveParams.XDrive.Stiffness, InDriveParams.YDrive.Stiffness, InDriveParams.ZDrive.Stiffness));
			Constraint->SetLinearDriveDamping(Chaos::ConstraintSettings::LinearDriveDampingScale() * Chaos::FVec3(InDriveParams.XDrive.Damping, InDriveParams.YDrive.Damping, InDriveParams.ZDrive.Damping));
			Constraint->SetLinearDriveMaxForce(Chaos::FVec3(InDriveParams.XDrive.MaxForce, InDriveParams.YDrive.MaxForce, InDriveParams.ZDrive.MaxForce));
		}
	}
}

void FPhysInterface_Chaos::UpdateAngularDrive_AssumesLocked(const FPhysicsConstraintHandle& InConstraintRef, const FAngularDriveConstraint& InDriveParams, bool InInitialize)
{
	if (InConstraintRef.IsValid() && InConstraintRef.Constraint->IsType(Chaos::EConstraintType::JointConstraintType))
	{
		if (Chaos::FJointConstraint* Constraint = static_cast<Chaos::FJointConstraint*>(InConstraintRef.Constraint))
		{
			Constraint->SetAngularSLerpPositionDriveEnabled(false);
			Constraint->SetAngularTwistPositionDriveEnabled(false);
			Constraint->SetAngularSwingPositionDriveEnabled(false);

			Constraint->SetAngularSLerpVelocityDriveEnabled(false);
			Constraint->SetAngularTwistVelocityDriveEnabled(false);
			Constraint->SetAngularSwingVelocityDriveEnabled(false);

			bool bPositionDriveEnabled = InDriveParams.IsOrientationDriveEnabled();
			if (bPositionDriveEnabled)
			{
				if (InDriveParams.AngularDriveMode == EAngularDriveMode::TwistAndSwing)
				{
					Constraint->SetAngularTwistPositionDriveEnabled(InDriveParams.TwistDrive.bEnablePositionDrive);
					Constraint->SetAngularSwingPositionDriveEnabled(InDriveParams.SwingDrive.bEnablePositionDrive);
				}
				else
				{
					Constraint->SetAngularSLerpPositionDriveEnabled(InDriveParams.SlerpDrive.bEnablePositionDrive);
				}

				if (InInitialize || FMath::IsNearlyEqual(Constraint->GetAngularPlasticityLimit(), (Chaos::FReal)FLT_MAX))
				{
					// Plastic joints should not be re-targeted after initialization. 
					Constraint->SetAngularDrivePositionTarget(Chaos::FRotation3(InDriveParams.OrientationTarget.Quaternion()));
				}
			}

			bool bVelocityDriveEnabled = InDriveParams.IsVelocityDriveEnabled();
			if (bVelocityDriveEnabled)
			{
				if (InDriveParams.AngularDriveMode == EAngularDriveMode::TwistAndSwing)
				{
					Constraint->SetAngularTwistVelocityDriveEnabled(InDriveParams.TwistDrive.bEnableVelocityDrive);
					Constraint->SetAngularSwingVelocityDriveEnabled(InDriveParams.SwingDrive.bEnableVelocityDrive);
				}
				else
				{
					Constraint->SetAngularSLerpVelocityDriveEnabled(InDriveParams.SlerpDrive.bEnableVelocityDrive);
				}

				if (!FMath::IsNearlyEqual(Constraint->GetAngularPlasticityLimit(), (Chaos::FReal)FLT_MAX))
				{
					// Plasticity requires a zero relative velocity.
					if (!Constraint->GetAngularDriveVelocityTarget().IsZero())
					{
						Constraint->SetAngularDriveVelocityTarget(FVector(ForceInitToZero));
					}
				}
				else
				{
					Constraint->SetAngularDriveVelocityTarget(InDriveParams.AngularVelocityTarget * 2.0f * UE_PI); // Rev/s to Rad/s
				}
			}

			Constraint->SetAngularDriveForceMode(InDriveParams.bAccelerationMode ? Chaos::EJointForceMode::Acceleration : Chaos::EJointForceMode::Force);
			if (InDriveParams.AngularDriveMode == EAngularDriveMode::TwistAndSwing)
			{
				Constraint->SetAngularDriveStiffness(Chaos::ConstraintSettings::AngularDriveStiffnessScale() * Chaos::FVec3(InDriveParams.TwistDrive.Stiffness, InDriveParams.SwingDrive.Stiffness, InDriveParams.SwingDrive.Stiffness));
				Constraint->SetAngularDriveDamping(Chaos::ConstraintSettings::AngularDriveDampingScale() * Chaos::FVec3(InDriveParams.TwistDrive.Damping, InDriveParams.SwingDrive.Damping, InDriveParams.SwingDrive.Damping));
				Constraint->SetAngularDriveMaxTorque(Chaos::FVec3(InDriveParams.TwistDrive.MaxForce, InDriveParams.SwingDrive.MaxForce, InDriveParams.SwingDrive.MaxForce));
			}
			else
			{
				Constraint->SetAngularDriveStiffness(Chaos::ConstraintSettings::AngularDriveStiffnessScale() * Chaos::FVec3(InDriveParams.SlerpDrive.Stiffness));
				Constraint->SetAngularDriveDamping(Chaos::ConstraintSettings::AngularDriveDampingScale() * Chaos::FVec3(InDriveParams.SlerpDrive.Damping));
				Constraint->SetAngularDriveMaxTorque(Chaos::FVec3(InDriveParams.SlerpDrive.MaxForce, InDriveParams.SlerpDrive.MaxForce, InDriveParams.SlerpDrive.MaxForce));
			}
		}
	}
}

void FPhysInterface_Chaos::UpdateDriveTarget_AssumesLocked(const FPhysicsConstraintHandle& InConstraintRef, const FLinearDriveConstraint& InLinDrive, const FAngularDriveConstraint& InAngDrive, bool InInitialize)
{
	if (InConstraintRef.IsValid())
	{
		UpdateLinearDrive_AssumesLocked(InConstraintRef, InLinDrive, InInitialize);
		UpdateAngularDrive_AssumesLocked(InConstraintRef, InAngDrive, InInitialize);
	}
}

bool FPhysInterface_Chaos::ExecuteOnUnbrokenConstraintReadOnly(const FPhysicsConstraintHandle& InConstraintRef, TFunctionRef<void(const FPhysicsConstraintHandle&)> Func)
{
    if (!IsBroken(InConstraintRef))
    {
		FScopedSceneLock_Chaos SceneLock(&InConstraintRef, EPhysicsInterfaceScopedLockType::Read);
        Func(InConstraintRef);
        return true;
    }
    return false;
}

bool FPhysInterface_Chaos::ExecuteOnUnbrokenConstraintReadWrite(const FPhysicsConstraintHandle& InConstraintRef, TFunctionRef<void(const FPhysicsConstraintHandle&)> Func)
{
    if (!IsBroken(InConstraintRef))
    {
		FScopedSceneLock_Chaos SceneLock(&InConstraintRef, EPhysicsInterfaceScopedLockType::Write);
        Func(InConstraintRef);
        return true;
    }
    return false;
}

bool FPhysInterface_Chaos::ExecuteRead(const FPhysicsActorHandle& InActorReference, TFunctionRef<void(const FPhysicsActorHandle& Actor)> InCallable)
{
	if(InActorReference)
	{
		FScopedSceneLock_Chaos SceneLock(InActorReference, EPhysicsInterfaceScopedLockType::Read);

		InCallable(InActorReference);
		return true;
	}
	return false;
}

bool FPhysInterface_Chaos::ExecuteRead(USkeletalMeshComponent* InMeshComponent, TFunctionRef<void()> InCallable)
{
	FScopedSceneLock_Chaos SceneLock(InMeshComponent, EPhysicsInterfaceScopedLockType::Read);
	InCallable();
	return true;
}

bool FPhysInterface_Chaos::ExecuteRead(const FPhysicsActorHandle& InActorReferenceA, const FPhysicsActorHandle& InActorReferenceB, TFunctionRef<void(const FPhysicsActorHandle& ActorA, const FPhysicsActorHandle& ActorB)> InCallable)
{
	FScopedSceneLock_Chaos SceneLock(InActorReferenceA, InActorReferenceB, EPhysicsInterfaceScopedLockType::Read);
	InCallable(InActorReferenceA, InActorReferenceB);
	return true;
}

bool FPhysInterface_Chaos::ExecuteRead(const FPhysicsConstraintHandle& InConstraintRef, TFunctionRef<void(const FPhysicsConstraintHandle& Constraint)> InCallable)
{
	if(InConstraintRef.IsValid())
	{
		FScopedSceneLock_Chaos SceneLock(&InConstraintRef, EPhysicsInterfaceScopedLockType::Read);
		InCallable(InConstraintRef);
		return true;
	}

	return false;
}

bool FPhysInterface_Chaos::ExecuteRead(FPhysScene* InScene, TFunctionRef<void()> InCallable)
{
	if(InScene)
	{
		FScopedSceneLock_Chaos SceneLock(InScene, EPhysicsInterfaceScopedLockType::Read);
		InCallable();
		return true;
	}

	return false;
}

bool FPhysInterface_Chaos::ExecuteRead(Chaos::FPhysicsObject* InObjectA, Chaos::FPhysicsObject* InObjectB, TFunctionRef<void(Chaos::FPhysicsObject* InObjectA, Chaos::FPhysicsObject* InObjectB)> InCallable)
{
	FScopedSceneLock_Chaos SceneLock(InObjectA, InObjectB, EPhysicsInterfaceScopedLockType::Read);
	InCallable(InObjectA, InObjectB);
	return true;
}

bool FPhysInterface_Chaos::ExecuteWrite(const FPhysicsActorHandle& InActorReference, TFunctionRef<void(const FPhysicsActorHandle& Actor)> InCallable)
{
	if (InActorReference)
	{
		FScopedSceneLock_Chaos SceneLock(InActorReference, EPhysicsInterfaceScopedLockType::Write);
		InCallable(InActorReference);
		return true;
	}

	return false;
}

bool FPhysInterface_Chaos::ExecuteWrite(FPhysicsActorHandle& InActorReference, TFunctionRef<void(FPhysicsActorHandle& Actor)> InCallable)
{
	if (InActorReference)
	{
		FScopedSceneLock_Chaos SceneLock(InActorReference, EPhysicsInterfaceScopedLockType::Write);
		InCallable(InActorReference);
		return true;
	}

	return false;
}

bool FPhysInterface_Chaos::ExecuteWrite(const FPhysicsActorHandle& InActorReferenceA, const FPhysicsActorHandle& InActorReferenceB, TFunctionRef<void(const FPhysicsActorHandle& ActorA, const FPhysicsActorHandle& ActorB)> InCallable)
{
	FScopedSceneLock_Chaos SceneLock(InActorReferenceA, InActorReferenceB, EPhysicsInterfaceScopedLockType::Write);
	InCallable(InActorReferenceA, InActorReferenceB);
	return true;
}

bool FPhysInterface_Chaos::ExecuteWrite(const FPhysicsConstraintHandle& InConstraintRef, TFunctionRef<void(const FPhysicsConstraintHandle& Constraint)> InCallable)
{
	if (InConstraintRef.IsValid())
	{
		FScopedSceneLock_Chaos SceneLock(&InConstraintRef, EPhysicsInterfaceScopedLockType::Write);
		InCallable(InConstraintRef);
		return true;
	}

	return false;
}

bool FPhysInterface_Chaos::ExecuteWrite(USkeletalMeshComponent* InMeshComponent, TFunctionRef<void()> InCallable)
{
	FScopedSceneLock_Chaos SceneLock(InMeshComponent, EPhysicsInterfaceScopedLockType::Write);
	InCallable();
	return true;
}

bool FPhysInterface_Chaos::ExecuteWrite(FPhysScene* InScene, TFunctionRef<void()> InCallable)
{
	if(InScene)
	{
		FScopedSceneLock_Chaos SceneLock(InScene, EPhysicsInterfaceScopedLockType::Write);
		InCallable();
		return true;
	}

	return false;
}

bool FPhysInterface_Chaos::ExecuteWrite(FPhysScene* InScene, TFunctionRef<void(FPhysScene* Scene)> InCallable)
{
	if (InScene)
	{
		FScopedSceneLock_Chaos SceneLock(InScene, EPhysicsInterfaceScopedLockType::Write);
		InCallable(InScene);
		return true;
	}

	return false;
}

bool FPhysInterface_Chaos::ExecuteWrite(Chaos::FPhysicsObject* InObjectA, Chaos::FPhysicsObject* InObjectB, TFunctionRef<void(Chaos::FPhysicsObject* InObjectA, Chaos::FPhysicsObject* InObjectB)> InCallable)
{
	FScopedSceneLock_Chaos SceneLock(InObjectA, InObjectB, EPhysicsInterfaceScopedLockType::Write);
	InCallable(InObjectA, InObjectB);
	return true;
}

void FPhysInterface_Chaos::ExecuteShapeWrite(FBodyInstance* InInstance, FPhysicsShapeHandle& InShape, TFunctionRef<void(FPhysicsShapeHandle& InShape)> InCallable)
{
	if(InInstance && InShape.IsValid())
	{
		FScopedSceneLock_Chaos SceneLock(InInstance->GetPhysicsActor(), EPhysicsInterfaceScopedLockType::Write);
		InCallable(InShape);
	}
}


bool FPhysInterface_Chaos::ExecuteWrite(FPhysicsActorHandle& InActorReference, TFunctionRef<void(FPhysicsActorHandle& Actor)> InCallable, EPhysicsInterfaceScopedThreadContext ThreadContext, EPhysicsInterfaceScopedTransactionMode TransactionMode)
{
	if (InActorReference)
	{
		FScopedSceneLockWithContext_Chaos SceneLock(InActorReference, EPhysicsInterfaceScopedLockType::Write, ThreadContext, TransactionMode);
		InCallable(InActorReference);
		return true;
	}

	return false;
}


bool FPhysInterface_Chaos::ExecuteRead(FPhysicsActorHandle& InActorReference, TFunctionRef<void(FPhysicsActorHandle& Actor)> InCallable, EPhysicsInterfaceScopedThreadContext ThreadContext)
{
	if (InActorReference)
	{
		FScopedSceneLockWithContext_Chaos SceneLock(InActorReference, EPhysicsInterfaceScopedLockType::Read, ThreadContext, EPhysicsInterfaceScopedTransactionMode::Normal);
		InCallable(InActorReference);
		return true;
	}

	return false;
}

bool FPhysInterface_Chaos::ExecuteWrite(const FPhysicsConstraintHandle& InConstraintRef, TFunctionRef<void(const FPhysicsConstraintHandle& Constraint)> InCallable, EPhysicsInterfaceScopedThreadContext ThreadContext, EPhysicsInterfaceScopedTransactionMode TransactionMode)
{
	if (InConstraintRef.IsValid())
	{
		FScopedSceneLockWithContext_Chaos SceneLock(&InConstraintRef, EPhysicsInterfaceScopedLockType::Write, ThreadContext, TransactionMode);
		InCallable(InConstraintRef);
		return true;
	}

	return false;
}

bool FPhysInterface_Chaos::ExecuteRead(const FPhysicsConstraintHandle& InConstraintRef, TFunctionRef<void(const FPhysicsConstraintHandle& Constraint)> InCallable, EPhysicsInterfaceScopedThreadContext ThreadContext)
{
	if (InConstraintRef.IsValid())
	{
		FScopedSceneLockWithContext_Chaos SceneLock(&InConstraintRef, EPhysicsInterfaceScopedLockType::Read, ThreadContext, EPhysicsInterfaceScopedTransactionMode::Normal);
		InCallable(InConstraintRef);
		return true;
	}

	return false;
}

FPhysicsShapeHandle FPhysInterface_Chaos::CreateShape(physx::PxGeometry* InGeom, bool bSimulation, bool bQuery, UPhysicalMaterial* InSimpleMaterial, TArray<UPhysicalMaterial*>* InComplexMaterials)
{
	// #todo : Implement
	// @todo(mlentine): Should we be doing anything with the InGeom here?
    FPhysicsActorHandle NewActor = nullptr;
	return { nullptr, NewActor };
}

const FBodyInstance* FPhysInterface_Chaos::ShapeToOriginalBodyInstance(const FBodyInstance* InCurrentInstance, const Chaos::FPerShapeData* InShape)
{
	//question: this is identical to physx version, should it be in body instance?
	check(InCurrentInstance);
	check(InShape);

	const FBodyInstance* TargetInstance = InCurrentInstance->WeldParent ? InCurrentInstance->WeldParent : InCurrentInstance;
	const FBodyInstance* OutInstance = TargetInstance;

	if (const TMap<FPhysicsShapeHandle, FBodyInstance::FWeldInfo>* WeldInfo = InCurrentInstance->GetCurrentWeldInfo())
	{
		for (const TPair<FPhysicsShapeHandle, FBodyInstance::FWeldInfo>& Pair : *WeldInfo)
		{
			if (Pair.Key.Shape == InShape)
			{
				TargetInstance = Pair.Value.ChildBI;
			}
		}
	}

	return TargetInstance;
}



void FPhysInterface_Chaos::AddGeometry(const FPhysicsActorHandle& InActor, const FGeometryAddParams& InParams, TArray<FPhysicsShapeHandle>* OutOptShapes)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPhysInterface_Chaos::AddGeometry);
	LLM_SCOPE(ELLMTag::ChaosGeometry);

	// @todo(chaos): we should not be creating unique geometry per actor
	// @todo(chaos): we are creating the Shapes array twice. Once here and again in SetGeometry or MergeGeometry. Fix this.
	TArray<Chaos::FImplicitObjectPtr> Geoms;
	Chaos::FShapesArray Shapes;
	ChaosInterface::CreateGeometry(InParams, Geoms, Shapes);

	if (InActor && Geoms.Num())
	{
		for (TUniquePtr<Chaos::FPerShapeData>& Shape : Shapes)
		{
			FPhysicsShapeHandle NewHandle(Shape.Get(), InActor);
			if (OutOptShapes)
			{
				OutOptShapes->Add(NewHandle);
			}

			FBodyInstance::ApplyMaterialToShape_AssumesLocked(NewHandle, InParams.SimpleMaterial, InParams.ComplexMaterials, &InParams.ComplexMaterialMasks);

			//TArrayView<UPhysicalMaterial*> SimpleView = MakeArrayView(&(const_cast<UPhysicalMaterial*>(InParams.SimpleMaterial)), 1);
			//FPhysInterface_Chaos::SetMaterials(NewHandle, InParams.ComplexMaterials.Num() > 0 ? InParams.ComplexMaterials : SimpleView);
		}

		// NOTE: Both MergeGeometry and SetGeometry will extend the ShapesInstances array to contain enough elements for
		// each geometry in the Union. However the shape data will not have been filled in, hence the call to MergeShapeInstance at the end.
		// todo: we should not be creating unique geometry per actor
		{
			if (InActor->GetGameThreadAPI().GetGeometry())
			{
				// Geometry already exists - combine new geometry with the existing
				// NOTE: We do not need to set the AllowBVH flag because it will be cloned (see below)
				InActor->GetGameThreadAPI().MergeGeometry(MoveTemp(Geoms));
			}
			else
			{
				// We always have a union so we can support any future welding operations. (Non-trivial converting the SharedPtr to UniquePtr).
				// NOTE: The root union always supports BVH (if there are enough shapes) and is the only Union in the hierarchy that is allowed 
				// to do so, but we don't create it here because that makes welding even more expensive (bodies are welded one by one). 
				// Search for SetAllowBVH to see where the BVH is enabled.
				Chaos::FImplicitObjectPtr Union = MakeImplicitObjectPtr<Chaos::FImplicitObjectUnion>(MoveTemp(Geoms));
				InActor->GetGameThreadAPI().SetGeometry(MoveTemp(Union));
			}
		}

		// Update the newly added shapes with the collision filters, materials etc
		// NOTE: MergeShapes overwrites the last N shapes (see comments above)
		InActor->GetGameThreadAPI().MergeShapesArray(MoveTemp(Shapes));
	}
}

void FPhysInterface_Chaos::SetMaterials(const FPhysicsShapeHandle& InShape, const TArrayView<UPhysicalMaterial*> InMaterials)
{
	// Build a list of handles to store on the shape
	TArray<Chaos::FMaterialHandle> NewMaterialHandles;
	NewMaterialHandles.Reserve(InMaterials.Num());

	for(UPhysicalMaterial* UnrealMaterial : InMaterials)
	{
		NewMaterialHandles.Add(UnrealMaterial->GetPhysicsMaterial());
	}

	InShape.Shape->SetMaterials(MoveTemp(NewMaterialHandles));
}

void FPhysInterface_Chaos::SetMaterials(const FPhysicsShapeHandle& InShape, const TArrayView<UPhysicalMaterial*> InMaterials, const TArrayView<FPhysicalMaterialMaskParams>& InMaterialMasks)
{
	SetMaterials(InShape, InMaterials);

	if (InMaterialMasks.Num() > 0)
	{
		// Build a list of handles to store on the shape
		TArray<Chaos::FMaterialMaskHandle> NewMaterialMaskHandles;
		TArray<uint32> NewMaterialMaskMaps;
		TArray<Chaos::FMaterialHandle> NewMaterialMaskMaterialHandles;

		NewMaterialMaskHandles.Reserve(InMaterialMasks.Num());
		NewMaterialMaskMaps.Reserve(InMaterialMasks.Num() * EPhysicalMaterialMaskColor::MAX);

		int MaskMapMatIdx = 0;

		for(FPhysicalMaterialMaskParams& MaterialMaskData : InMaterialMasks)
		{
			if(MaterialMaskData.PhysicalMaterialMask && ensure(MaterialMaskData.PhysicalMaterialMap))
			{
				NewMaterialMaskHandles.Add(MaterialMaskData.PhysicalMaterialMask->GetPhysicsMaterialMask());
				for(int i = 0; i < EPhysicalMaterialMaskColor::MAX; i++)
				{
					if(UPhysicalMaterial* MapMat = MaterialMaskData.PhysicalMaterialMap->GetPhysicalMaterialFromMap(i))
					{
						NewMaterialMaskMaps.Emplace(MaskMapMatIdx);
						MaskMapMatIdx++;
					} 
					else
					{
						NewMaterialMaskMaps.Emplace(INDEX_NONE);
					}
				}
			} 
			else
			{
				NewMaterialMaskHandles.Add(Chaos::FMaterialMaskHandle());
				for(int i = 0; i < EPhysicalMaterialMaskColor::MAX; i++)
				{
					NewMaterialMaskMaps.Emplace(INDEX_NONE);
				}
			}
		}
		
		if (MaskMapMatIdx > 0)
		{
			NewMaterialMaskMaterialHandles.Reserve(MaskMapMatIdx);

			uint32 Offset = 0;

			for (FPhysicalMaterialMaskParams& MaterialMaskData : InMaterialMasks)
			{
				if (MaterialMaskData.PhysicalMaterialMask)
				{
					for (int i = 0; i < EPhysicalMaterialMaskColor::MAX; i++)
					{
						if (UPhysicalMaterial* MapMat = MaterialMaskData.PhysicalMaterialMap->GetPhysicalMaterialFromMap(i))
						{
							NewMaterialMaskMaterialHandles.Add(MapMat->GetPhysicsMaterial());
						}
					}
				}
			}
		}

		InShape.Shape->SetMaterialMasks(MoveTemp(NewMaterialMaskHandles));
		InShape.Shape->SetMaterialMaskMaps(MoveTemp(NewMaterialMaskMaps));
		InShape.Shape->SetMaterialMaskMapMaterials(MoveTemp(NewMaterialMaskMaterialHandles));
	}
}

void FinishSceneStat()
{
}

bool FPhysInterface_Chaos::LineTrace_Geom(FHitResult& OutHit, const FBodyInstance* InInstance, const FVector& WorldStart, const FVector& WorldEnd, bool bTraceComplex, bool bExtractPhysMaterial)
{
	using namespace ChaosInterface;

	// Need an instance to trace against
	check(InInstance);

	OutHit.TraceStart = WorldStart;
	OutHit.TraceEnd = WorldEnd;

	bool bHitSomething = false;

	const FVector Delta = WorldEnd - WorldStart;
	const float DeltaMag = Delta.Size();
	if (DeltaMag > UE_KINDA_SMALL_NUMBER)
	{
		{
			// #PHYS2 Really need a concept for "multi" locks here - as we're locking ActorRef but not TargetInstance->ActorRef
			FPhysicsCommand::ExecuteRead(InInstance->GetPhysicsActor(), [&](const FPhysicsActorHandle& Actor)
			{
				// If we're welded then the target instance is actually our parent
				const FBodyInstance* TargetInstance = InInstance->WeldParent ? InInstance->WeldParent : InInstance;
				if(FPhysicsActorHandle RigidBody = TargetInstance->GetPhysicsActor())
				{
					FRaycastHit BestHit;
					BestHit.Distance = FLT_MAX;

					// Get all the shapes from the actor
					PhysicsInterfaceTypes::FInlineShapeArray Shapes;
					const int32 NumShapes = FillInlineShapeArray_AssumesLocked(Shapes, Actor);

					const FTransform WorldTM(RigidBody->GetGameThreadAPI().R(), RigidBody->GetGameThreadAPI().X());
					const FVector LocalStart = WorldTM.InverseTransformPositionNoScale(WorldStart);
					const FVector LocalDelta = WorldTM.InverseTransformVectorNoScale(Delta);

					// Iterate over each shape
					for (int32 ShapeIdx = 0; ShapeIdx < NumShapes; ShapeIdx++)
					{
						// #PHYS2 - SHAPES - Resolve this single cast case
						FPhysicsShapeReference_Chaos& ShapeRef = Shapes[ShapeIdx];
						Chaos::FPerShapeData* Shape = ShapeRef.Shape;
						check(Shape);

						if (TargetInstance->IsShapeBoundToBody(ShapeRef) == false)
						{
							continue;
						}

						// Filter so we trace against the right kind of collision
						const Chaos::Filter::FShapeFilterData& ShapeFilter = Shape->GetShapeFilterData();
						const bool bShapeIsComplex = ShapeFilter.HasFlag(Chaos::EFilterFlags::ComplexCollision);
						const bool bShapeIsSimple = ShapeFilter.HasFlag(Chaos::EFilterFlags::SimpleCollision);
						if ((bTraceComplex && bShapeIsComplex) || (!bTraceComplex && bShapeIsSimple))
						{

							Chaos::FReal Distance;
							Chaos::FVec3 LocalPosition;
							Chaos::FVec3 LocalNormal;

							int32 FaceIndex;
							if (Shape->GetGeometry()->Raycast(LocalStart, LocalDelta / DeltaMag, DeltaMag, 0, Distance, LocalPosition, LocalNormal, FaceIndex))
							{
								if (Distance < BestHit.Distance)
								{
									BestHit.Distance = Distance;
									BestHit.WorldNormal = LocalNormal;	//will convert to world when best is chosen
									BestHit.WorldPosition = LocalPosition;
									BestHit.Shape = Shape;
									BestHit.Actor = Actor->GetParticle_LowLevel();
									BestHit.FaceIndex = FaceIndex;
								}
							}
						}
					}

					if (BestHit.Distance < FLT_MAX)
					{
						BestHit.WorldNormal = WorldTM.TransformVectorNoScale(BestHit.WorldNormal);
						BestHit.WorldPosition = WorldTM.TransformPositionNoScale(BestHit.WorldPosition);
						SetFlags(BestHit, EHitFlags::Distance | EHitFlags::Normal | EHitFlags::Position);

						// Verify the hit is made by doing an overlap test.
						const Chaos::Filter::FQueryFilterData QueryFilter = Chaos::Filter::FQueryTraceFilterBuilder::BuildOverlapAll();

						FTransform StartTM(WorldStart);
						const UWorld* World = InInstance->GetWorld();

						ConvertQueryImpactHit(World, BestHit, OutHit, DeltaMag, QueryFilter, WorldStart, WorldEnd, nullptr, StartTM, true, bExtractPhysMaterial);
						bHitSomething = true;
					}
				}
			});
		}
	}

	return bHitSomething;
}

bool FPhysInterface_Chaos::Sweep_Geom(FHitResult& OutHit, const FBodyInstance* InInstance, const FVector& InStart, const FVector& InEnd, const FQuat& InShapeRotation, const FCollisionShape& InShape, bool bSweepComplex)
{
	using namespace ChaosInterface;

	bool bSweepHit = false;

	if (InShape.IsNearlyZero())
	{
		bSweepHit = LineTrace_Geom(OutHit, InInstance, InStart, InEnd, bSweepComplex);
	}
	else
	{
		OutHit.TraceStart = InStart;
		OutHit.TraceEnd = InEnd;

		const FBodyInstance* TargetInstance = InInstance->WeldParent ? InInstance->WeldParent : InInstance;

		FPhysicsCommand::ExecuteRead(TargetInstance->GetPhysicsActor(), [&](const FPhysicsActorHandle& Actor)
		{
			if (Actor)
			{
				FPhysicsShapeAdapter ShapeAdapter(InShapeRotation, InShape);

				const FVector Delta = InEnd - InStart;
				const float DeltaMag = Delta.Size();
				
				const FTransform ActorTM(Actor->GetGameThreadAPI().R(), Actor->GetGameThreadAPI().X());

				FTransform StartTM(ShapeAdapter.GetGeomOrientation(), InStart);

				const Chaos::FVec3 Dir = DeltaMag > 0 ? (Delta / DeltaMag) : Chaos::FVec3(1, 0, 0);

				FSweepHit Hit;
				Hit.Distance = TNumericLimits<float>::Max();

				// Get all the shapes from the actor
				PhysicsInterfaceTypes::FInlineShapeArray Shapes;
				// #PHYS2 - SHAPES - Resolve this function to not use px stuff
				const int32 NumShapes = FillInlineShapeArray_AssumesLocked(Shapes, Actor); // #PHYS2 - Need a lock/execute here?

				// Iterate over each shape
				for (int32 ShapeIdx = 0; ShapeIdx < NumShapes; ShapeIdx++)
				{
					FPhysicsShapeReference_Chaos& ShapeRef = Shapes[ShapeIdx];
					Chaos::FPerShapeData* Shape = ShapeRef.Shape;
					check(Shape);

					// Skip shapes not bound to this instance
					if (!TargetInstance->IsShapeBoundToBody(ShapeRef))
					{
						continue;
					}

					// Filter so we trace against the right kind of collision
					const Chaos::Filter::FShapeFilterData& ShapeFilter = Shape->GetShapeFilterData();
					const bool bShapeIsComplex = ShapeFilter.HasFlag(Chaos::EFilterFlags::ComplexCollision);
					const bool bShapeIsSimple = ShapeFilter.HasFlag(Chaos::EFilterFlags::SimpleCollision);
					if ((bSweepComplex && bShapeIsComplex) || (!bSweepComplex && bShapeIsSimple))
					{
						//question: this is returning first result, is that valid? Keeping it the same as physx for now
						Chaos::FVec3 WorldPosition;
						Chaos::FVec3 WorldNormal;
						Chaos::FReal Distance;
						int32 FaceIdx;
						Chaos::FVec3 FaceNormal;

						// This is what it needs to be, update later...
						const FPhysicsGeometry& QueryGeometry = ShapeAdapter.GetGeometry();
						const bool bShapeHit = Chaos::Utilities::CastHelper(QueryGeometry, ActorTM,
							[&](const auto& Downcast, const auto& FullActorTM)
							{
								return Chaos::SweepQuery(*Shape->GetGeometry(), FullActorTM, Downcast, StartTM, Dir, DeltaMag, Distance, WorldPosition, WorldNormal, FaceIdx, FaceNormal, 0.f, true);
							}
						);

						// we should eventually have the Hit structure to use a Chaos::FReal equivalent instead;
						const float FloatDistance = static_cast<float>(Distance);
						if (bShapeHit && FloatDistance < Hit.Distance)
						{
							// we just like to make sure if the hit is made
							const Chaos::Filter::FQueryFilterData QueryFilter = Chaos::Filter::FQueryTraceFilterBuilder::BuildOverlapAll();

							// we don't get Shape information when we access via PShape, so I filled it up
							Hit.Shape = Shape;
							Hit.Actor = ShapeRef.ActorRef ? ShapeRef.ActorRef->GetParticle_LowLevel() : nullptr;
							Hit.WorldPosition = WorldPosition;
							Hit.WorldNormal = WorldNormal;
							Hit.Distance = FloatDistance;
							Hit.FaceIndex = FaceIdx;
							Hit.FaceNormal = FaceNormal;

							// Always compute the best face index. This used to check for non-initial overlap hits, however there's no reason to do that as we should always have valid data.
							Hit.FaceIndex = FindFaceIndex(Hit, Dir);

							SetFlags(Hit, EHitFlags::Distance | EHitFlags::Normal | EHitFlags::Position | EHitFlags::FaceIndex);

							FTransform StartTransform(InStart);
							const UWorld* World = InInstance->GetWorld();

							ConvertQueryImpactHit(World, Hit, OutHit, DeltaMag, QueryFilter, InStart, InEnd, &QueryGeometry, StartTransform, true, false);
							bSweepHit = true;
						}
					}
				}
			}
		});
	}

	return bSweepHit;
}

bool Overlap_GeomInternal(const FBodyInstance* InInstance, const Chaos::FImplicitObject& InGeom, const FTransform& GeomTransform, FMTDResult* OutOptResult, bool bTraceComplex)
{
	const FBodyInstance* TargetInstance = InInstance->WeldParent ? InInstance->WeldParent : InInstance;
	FPhysicsActorHandle RigidBody = TargetInstance->GetPhysicsActor();

	if (!RigidBody)
	{
		return false;
	}

	// Get all the shapes from the actor
	PhysicsInterfaceTypes::FInlineShapeArray Shapes;
	const int32 NumShapes = FillInlineShapeArray_AssumesLocked(Shapes, RigidBody);

	const FTransform ActorTM(RigidBody->GetGameThreadAPI().R(), RigidBody->GetGameThreadAPI().X());

	if (OutOptResult)
	{
		OutOptResult->Distance = 0.0;
	}

	bool bHasOverlap = false;
	// Iterate over each shape
	for (int32 ShapeIdx = 0; ShapeIdx < NumShapes; ++ShapeIdx)
	{
		FPhysicsShapeReference_Chaos& ShapeRef = Shapes[ShapeIdx];
		const Chaos::FPerShapeData* Shape = ShapeRef.Shape;
		check(Shape);

		const Chaos::Filter::FShapeFilterData& ShapeFilter = Shape->GetShapeFilterData();
		const bool bShapeIsComplex = ShapeFilter.HasFlag(Chaos::EFilterFlags::ComplexCollision);
		const bool bShapeIsSimple = ShapeFilter.HasFlag(Chaos::EFilterFlags::SimpleCollision);
		if ((bTraceComplex && bShapeIsComplex) || (!bTraceComplex && bShapeIsSimple))
		{
			if (TargetInstance->IsShapeBoundToBody(ShapeRef))
			{
				if (OutOptResult)
				{
					Chaos::FMTDInfo MTDInfo;
					if (Chaos::Utilities::CastHelper(InGeom, GeomTransform, [&](const auto& Downcast, const auto& FullGeomTransform) { return Chaos::OverlapQuery(*Shape->GetGeometry(), ActorTM, Downcast, FullGeomTransform, /*Thickness=*/0, &MTDInfo); }))
					{
						bHasOverlap = true;
						if (MTDInfo.Penetration > OutOptResult->Distance)
						{
							OutOptResult->Distance = MTDInfo.Penetration;
							OutOptResult->Direction = MTDInfo.Normal;
						}
					}
				}
				else	//question: why do we even allow user to not pass in MTD info?
				{
					if (Chaos::Utilities::CastHelper(InGeom, GeomTransform, [&](const auto& Downcast, const auto& FullGeomTransform) { return Chaos::OverlapQuery(*Shape->GetGeometry(), ActorTM, Downcast, FullGeomTransform); }))
					{
						return true;
					}
				}

			}
		}
	}

	return bHasOverlap;
}

bool FPhysInterface_Chaos::Overlap_Geom(const FBodyInstance* InBodyInstance, const FPhysicsGeometryCollection& InGeometry, const FTransform& InShapeTransform, FMTDResult* OutOptResult, bool bTraceComplex)
{
	return Overlap_GeomInternal(InBodyInstance, InGeometry.GetGeometry(), InShapeTransform, OutOptResult, bTraceComplex);
}

bool FPhysInterface_Chaos::Overlap_Geom(const FBodyInstance* InBodyInstance, const FCollisionShape& InCollisionShape, const FQuat& InShapeRotation, const FTransform& InShapeTransform, FMTDResult* OutOptResult, bool bTraceComplex)
{
	FPhysicsShapeAdapter Adaptor(InShapeRotation, InCollisionShape);
	return Overlap_GeomInternal(InBodyInstance, Adaptor.GetGeometry(), Adaptor.GetGeomPose(InShapeTransform.GetTranslation()), OutOptResult, bTraceComplex);
}

bool FPhysInterface_Chaos::GetSquaredDistanceToBody(const FBodyInstance* InInstance, const FVector& InPoint, float& OutDistanceSquared, FVector* OutOptPointOnBody)
{
	if (OutOptPointOnBody)
	{
		*OutOptPointOnBody = InPoint;
		OutDistanceSquared = 0.f;
	}

	float ReturnDistance = -1.f;
	float MinPhi = UE_BIG_NUMBER;
	bool bFoundValidBody = false;
	bool bEarlyOut = true;

	const FBodyInstance* UseBI = InInstance->WeldParent ? InInstance->WeldParent : InInstance;
	const FTransform BodyTM = UseBI->GetUnrealWorldTransform();
	const FVector LocalPoint = BodyTM.InverseTransformPositionNoScale(InPoint);

	FPhysicsCommand::ExecuteRead(UseBI->GetPhysicsActor(), [&](const FPhysicsActorHandle& Actor)
	{
		bEarlyOut = false;

		TArray<FPhysicsShapeReference_Chaos> Shapes;
		UseBI->GetAllShapes_AssumesLocked(Shapes);
		for (const FPhysicsShapeReference_Chaos& Shape : Shapes)
		{
			if (!Shape.IsValid())
			{
				continue;
			}

			if (InInstance->IsShapeBoundToBody(Shape) == false)	//skip welded shapes that do not belong to us
			{
				continue;
			}

			ECollisionShapeType GeomType = FPhysicsInterface::GetShapeType(Shape);

			if (!Shape.GetGeometry().IsConvex())
			{
				// Type unsupported for this function, but some other shapes will probably work. 
				continue;
			}

			bFoundValidBody = true;

			Chaos::FVec3 Normal;
			const float Phi = Shape.Shape->GetGeometry()->PhiWithNormal(LocalPoint, Normal);
			if (Phi <= 0)
			{
				OutDistanceSquared = 0;
				if (OutOptPointOnBody)
				{
					*OutOptPointOnBody = InPoint;
				}
				break;
			}
			else if (Phi < MinPhi)
			{
				MinPhi = Phi;
				OutDistanceSquared = Phi * Phi;
				if (OutOptPointOnBody)
				{
					const Chaos::FVec3 LocalClosestPoint = LocalPoint - Phi * Normal;
					*OutOptPointOnBody = BodyTM.TransformPositionNoScale(LocalClosestPoint);
				}
			}
		}
	});

	if (!bFoundValidBody && !bEarlyOut)
	{
		UE_LOGF(LogPhysics, Verbose, "GetDistanceToBody: Component (%ls) has no simple collision and cannot be queried for closest point.", InInstance->OwnerComponent.Get() ? *(InInstance->OwnerComponent->GetPathName()) : TEXT("NONE"));
	}

	return bFoundValidBody;
}

uint32 GetTriangleMeshExternalFaceIndex(const Chaos::FImplicitObject* Geom, uint32 InternalFaceIndex)
{
	using namespace Chaos;
	uint8 OuterType = Geom->GetType();
	uint8 InnerType = GetInnerType(OuterType);
	if (ensure(InnerType == ImplicitObjectType::TriangleMesh))
	{
		const FTriangleMeshImplicitObject* TriangleMesh = nullptr;

		if (IsScaled(OuterType))
		{
			const TImplicitObjectScaled<FTriangleMeshImplicitObject>& ScaledTriangleMesh = Geom->GetObjectChecked<TImplicitObjectScaled<FTriangleMeshImplicitObject>>();
			TriangleMesh = ScaledTriangleMesh.GetUnscaledObject();
		}
		else if(IsInstanced(OuterType))
		{
			TriangleMesh = Geom->GetObjectChecked<TImplicitObjectInstanced<FTriangleMeshImplicitObject>>().GetInstancedObject();
		}
		else
		{
			TriangleMesh = &Geom->GetObjectChecked<FTriangleMeshImplicitObject>();
		}

		return TriangleMesh->GetExternalFaceIndexFromInternal(InternalFaceIndex);
	}

	return -1;
}

uint32 GetTriangleMeshExternalFaceIndex(const FPhysicsShape& Shape, uint32 InternalFaceIndex)
{
	// NOTE: GetLeafGeometry will strip Transformed and Instanced wrappers (but not Scaled)
	return GetTriangleMeshExternalFaceIndex(Shape.GetLeafGeometry(), InternalFaceIndex);
}

void FPhysInterface_Chaos::CalculateMassPropertiesFromShapeCollection(Chaos::FMassProperties& OutProperties,const TArray<FPhysicsShapeHandle>& InShapes,float InDensityKGPerCM)
{
	ChaosInterface::CalculateMassPropertiesFromShapeCollection(OutProperties,InShapes,InDensityKGPerCM);
}

