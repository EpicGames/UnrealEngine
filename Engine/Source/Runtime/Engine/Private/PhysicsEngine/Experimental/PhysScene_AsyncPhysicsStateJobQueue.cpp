// Copyright Epic Games, Inc. All Rights Reserved.

#include "Physics/Experimental/PhysScene_AsyncPhysicsStateJobQueue.h"
#include "Physics/Experimental/PhysScene_Chaos.h"
#include "PhysicsEngine/BodySetup.h"
#include "Engine/World.h"
#include "Components/ActorComponent.h"
#include "GameFramework/WorldSettings.h"

namespace Chaos
{
	static float GAsyncPhysicsStateTaskTimeBudgetMS = 0.0f;
	static FAutoConsoleVariableRef CVarAsyncPhysicsStateTaskTimeBudgetMS(
		TEXT("p.Chaos.AsyncPhysicsStateTask.TimeBudgetMS"),
		GAsyncPhysicsStateTaskTimeBudgetMS,
		TEXT("Maximum time budget (in milliseconds) for the async physics state task (0 = unlimited)"),
		ECVF_Default);

	static float GPriorityAsyncPhysicsStateTaskExtraTimeBudgetMS = 0.0f;
	static FAutoConsoleVariableRef CVarPriorityAsyncPhysicsStateTaskExtraTimeBudgetMS(
		TEXT("p.Chaos.AsyncPhysicsStateTask.PriorityExtraTimeBudgetMS"),
		GPriorityAsyncPhysicsStateTaskExtraTimeBudgetMS,
		TEXT("Additional time budget (in milliseconds) allocated to the async physics state task during high-priority loading or seamless travel."),
		ECVF_Default
	);
}

FPhysScene_AsyncPhysicsStateJobQueue::FScopedDeferAsyncPhysicsStateJobs::FScopedDeferAsyncPhysicsStateJobs(FPhysScene* InPhysScene)
: PhysScene(InPhysScene)
{
	if (PhysScene)
	{
		check(IsInGameThread());
		PhysScene->GetOrCreateAsyncPhysicsStateJobQueue().BeginDeferringNewJobs();
	}
}

FPhysScene_AsyncPhysicsStateJobQueue::FScopedDeferAsyncPhysicsStateJobs::~FScopedDeferAsyncPhysicsStateJobs()
{
	if (PhysScene)
	{
		check(IsInGameThread());
		PhysScene->GetOrCreateAsyncPhysicsStateJobQueue().EndDeferringNewJobs();
	}
}

FPhysScene_AsyncPhysicsStateJobQueue::FPhysScene_AsyncPhysicsStateJobQueue(FPhysScene* InPhysScene)
	: PhysScene(InPhysScene)
{
	PhysScene->GetOwningWorld()->OnAllLevelsChanged().AddRaw(this, &FPhysScene_AsyncPhysicsStateJobQueue::OnUpdateLevelStreamingDone);
}

FPhysScene_AsyncPhysicsStateJobQueue::~FPhysScene_AsyncPhysicsStateJobQueue()
{
	PhysScene->GetOwningWorld()->OnAllLevelsChanged().RemoveAll(this);

	check(DeferNewJobsScopeCount == 0);
	if (!IsCompleted())
	{
		Tick(true);
	}
	check(IsCompleted());

	ensure(RootCounts.IsEmpty());
	RootCounts.Empty();
}

void FPhysScene_AsyncPhysicsStateJobQueue::AddJob(const FJob& Job)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPhysScene_AsyncPhysicsStateJobQueue::AddJob);
	check(IsInGameThread());

	Job.Owner = this;
	Job.OnPreExecute_GameThread();

	if (DeferNewJobsScopeCount > 0)
	{
		DeferredJobs.Add(Job);
	}
	else
	{
		FWriteScopeLock Lock(JobsLock);
		JobsToExecute.Add(Job);
	}
}

void FPhysScene_AsyncPhysicsStateJobQueue::RemoveJob(const FJob& Job)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPhysScene_AsyncPhysicsStateJobQueue::RemoveJob);
	check(IsInGameThread());
	check(Job.IsValid());
	Job.Owner = this;

	bool bWaitForJobToComplete = false;
	bool bFoundJob = false;
	{
		FWriteScopeLock Lock(JobsLock);
		// First test in CompletedJobs, if found, leave JobsToExecute 
		// as it will be cleaned up by the async task.
		if (CompletedJobs.Remove(Job))
		{
			bFoundJob = true;
		}
		// If the job is executing, wait for it to complete
		else if (ExecutingJob == Job)
		{
			bFoundJob = true;
			bWaitForJobToComplete = true;
		}
		// Else it's safe to remove it from JobsToExecute
		else if (JobsToExecute.Remove(Job))
		{
			bFoundJob = true;
		}
		else if (DeferredJobs.Remove(Job))
		{
			bFoundJob = true;
		}
	}

	if (bFoundJob)
	{
		if (bWaitForJobToComplete)
		{
			// Wait for the async task
			AsyncJobTask.Wait();
			FWriteScopeLock Lock(JobsLock);
			// If job is completed, needs to be removed from CompletedJobs
			if (!CompletedJobs.Remove(Job))
			{
				// Else if it's still the executing job, finish executing it
				if (ExecutingJob == Job)
				{
					UE::FTimeout NeverTimeout(UE::FTimeout::Never());
					ExecutingJob->Execute(NeverTimeout);
					ExecutingJob.Reset();
					JobsToExecute.Remove(Job);
				}
			}
			check(!ExecutingJob.IsSet());
		}
		Job.OnPostExecute_GameThread();
	}
}

void FPhysScene_AsyncPhysicsStateJobQueue::AddReferencedObjects(FReferenceCollector& Collector)
{
	for (auto& Pair : RootCounts)
	{
		if (TObjectPtr<UObject>& Object = Pair.Key)
		{
			Collector.AddReferencedObject(Object);
		}
	}
}

void FPhysScene_AsyncPhysicsStateJobQueue::BeginDeferringNewJobs()
{
	check(IsInGameThread());
	++DeferNewJobsScopeCount;
}

void FPhysScene_AsyncPhysicsStateJobQueue::EndDeferringNewJobs()
{
	check(IsInGameThread());
	check(DeferNewJobsScopeCount > 0);
	--DeferNewJobsScopeCount;

	if (DeferNewJobsScopeCount == 0 && DeferredJobs.Num() > 0)
	{
		FWriteScopeLock Lock(JobsLock);
		JobsToExecute.Append(MoveTemp(DeferredJobs));
	}
}

bool FPhysScene_AsyncPhysicsStateJobQueue::IsCompleted() const
{
	FReadScopeLock Lock(JobsLock);
	const bool bNoAsyncJob = AsyncJobTask.IsCompleted() && JobsToExecute.IsEmpty() && !ExecutingJob.IsSet() && CompletedJobs.IsEmpty();
	const bool bNoDeferredJob = (DeferNewJobsScopeCount == 0) && DeferredJobs.IsEmpty();

	return bNoAsyncJob && bNoDeferredJob;
}

void FPhysScene_AsyncPhysicsStateJobQueue::OnUpdateLevelStreamingDone()
{
	++GameThreadEpoch;

	LaunchAsyncJobTask();
}

void FPhysScene_AsyncPhysicsStateJobQueue::LaunchAsyncJobTask()
{
	if (!AsyncJobTask.IsCompleted())
	{
		return;
	}

	{
		FReadScopeLock Lock(JobsLock);
		if (JobsToExecute.IsEmpty())
		{
			return;
		}
	}

	// Grant additional time if a high-priority load is in progress or the world is undergoing seamless travel.
	float ExtraTimeMS = 0.f;
	if (Chaos::GAsyncPhysicsStateTaskTimeBudgetMS > 0.f && Chaos::GPriorityAsyncPhysicsStateTaskExtraTimeBudgetMS > 0.f)
	{
		const UWorld* World = PhysScene->GetOwningWorld();
		if (AWorldSettings* WorldSettings = World ? World->GetWorldSettings(false, false) : nullptr)
		{
			if (WorldSettings->bHighPriorityLoading || WorldSettings->bHighPriorityLoadingLocal || (World && World->IsInSeamlessTravel()))
			{
				ExtraTimeMS = Chaos::GPriorityAsyncPhysicsStateTaskExtraTimeBudgetMS;
			}
		}
	}

	double TimeBudgetSeconds = Chaos::GAsyncPhysicsStateTaskTimeBudgetMS > 0 ? (Chaos::GAsyncPhysicsStateTaskTimeBudgetMS + ExtraTimeMS)/1000 : MAX_dbl;
	AsyncJobTask = UE::Tasks::Launch(UE_SOURCE_LOCATION, [this, TimeBudgetSeconds]() { ExecuteJobsAsync(TimeBudgetSeconds); }, UE::Tasks::ETaskPriority::BackgroundHigh);
}

void FPhysScene_AsyncPhysicsStateJobQueue::ExecuteJobsAsync(double TimeBudgetSeconds)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPhysScene_AsyncPhysicsStateJobQueue::ExecuteJobsAsync);
	UE::FTimeout Timeout(TimeBudgetSeconds - UsedAsyncTaskTimeBudgetSec);
	bool bWorkRemaining = false;

	int32 CompletedJobsCount = 0;
	int32 Index = 0;
	do
	{
		if (bIsBlocking)
		{
			Timeout = UE::FTimeout::Never();
		}
		else if (TaskEpoch != GameThreadEpoch)
		{
			// Reset Timeout
			TaskEpoch = GameThreadEpoch;
			Timeout = UE::FTimeout(TimeBudgetSeconds);
			UsedAsyncTaskTimeBudgetSec = 0;
		}
		
		{
			FWriteScopeLock Lock(JobsLock);
			if (!JobsToExecute.IsValidIndex(Index))
			{
				break;
			}
			check(!ExecutingJob.IsSet() || (ExecutingJob == JobsToExecute[Index]));
			ExecutingJob = JobsToExecute[Index++];
		}

		if (ExecutingJob->Execute(Timeout))
		{
			FWriteScopeLock Lock(JobsLock);
			CompletedJobs.Add(*ExecutingJob);
			ExecutingJob.Reset();
			++CompletedJobsCount;
		}
		else
		{
			break;
		}

		if (Timeout.IsExpired())
		{
			break;
		}
	} while (true);

	// Remove completed jobs
	{
		FWriteScopeLock Lock(JobsLock);
		JobsToExecute.RemoveAt(0, CompletedJobsCount);
	}

	UsedAsyncTaskTimeBudgetSec += Timeout.GetElapsedSeconds();
}

void FPhysScene_AsyncPhysicsStateJobQueue::Tick(bool bWaitForCompletion)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPhysScene_AsyncPhysicsStateJobQueue::Tick);
	check(IsInGameThread());
	if (IsCompleted())
	{
		return;
	}

	const bool bShouldWaitForCompletion = bWaitForCompletion ||
		PhysScene->GetOwningWorld()->GetIsInBlockTillLevelStreamingCompleted() ||
		PhysScene->GetOwningWorld()->GetShouldForceUnloadStreamingLevels() ||
		PhysScene->GetOwningWorld()->IsBeingCleanedUp();

	// Wait for tasks if the world is inside a blocking load
	if (bShouldWaitForCompletion)
	{
		// Tell the async task that there is no time limit anymore
		bIsBlocking = true;
		do
		{
			LaunchAsyncJobTask();
			AsyncJobTask.Wait();
			{
				FReadScopeLock Lock(JobsLock);
				if (JobsToExecute.IsEmpty())
				{
					break;
				}
			}
		} while (true);
		bIsBlocking = false;
	}

	TArray<FJob> CompletedJobsCopy;
	{
		FWriteScopeLock Lock(JobsLock);
		CompletedJobsCopy = MoveTemp(CompletedJobs);
	}

	if (!CompletedJobsCopy.IsEmpty())
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FPhysScene_AsyncPhysicsStateJobQueue::Tick_OnPostExecute_GameThread);
		for (const FJob& Job : CompletedJobsCopy)
		{
			Job.OnPostExecute_GameThread();
		}
	}
}

void FPhysScene_AsyncPhysicsStateJobQueue::AddRootObject(UObject* Obj)
{
	if (ensure(Obj))
	{
		int32& Count = RootCounts.FindOrAdd(Obj);
		++Count;
	}
}

void FPhysScene_AsyncPhysicsStateJobQueue::RemoveRootObject(UObject* Obj)
{
	if (!ensure(Obj))
	{
		return;
	}
	int32* CountPtr = RootCounts.Find(Obj);
	if (ensure(CountPtr))
	{
		--(*CountPtr);
		if (*CountPtr <= 0)
		{
			RootCounts.Remove(Obj);
		}
	}
}

void FPhysScene_AsyncPhysicsStateJobQueue::FJob::OnPreExecute_GameThread() const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPhysScene_AsyncPhysicsStateJobQueue::OnPreExecute_GameThread);

	RootedInputs.Reset();
	switch (Type)
	{
		case EJobType::CreatePhysicsState:
		{
			if (GEnableDeferredPhysicsCreation)
			{
				TSet<UBodySetup*> UniqueBodySetups;
				Processor->CollectBodySetupsWithPhysicsMeshesToCreate(UniqueBodySetups);
				if (UniqueBodySetups.Num())
				{
					TArray<UBodySetup*> BodySetups = UniqueBodySetups.Array();
					ParallelFor(TEXT("CreatePhysicsMeshes.PF"), BodySetups.Num(), 1, [this, &BodySetups](int32 Index) { BodySetups[Index]->CreatePhysicsMeshes(); });
				}
			}
			Processor->OnAsyncCreatePhysicsStateBegin_GameThread(RootedInputs);
		}
		break;
		case EJobType::DestroyPhysicsState:
		{
			Processor->OnAsyncDestroyPhysicsStateBegin_GameThread(RootedInputs);
		}
		break;
	}

	// Root referenced objects
	if (UObject* Obj = Processor->GetAsyncPhysicsStateObject())
	{
		RootedInputs.Add(Obj);
	}
	for (UObject* Obj : RootedInputs)
	{
		Owner->AddRootObject(Obj);
	}
}

bool FPhysScene_AsyncPhysicsStateJobQueue::FJob::Execute(UE::FTimeout& Timeout) const
{
	if (IsValid())
	{
		switch (Type)
		{
		case EJobType::CreatePhysicsState:
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(FPhysScene_AsyncPhysicsStateJobQueue::CreatePhysicsState);
			return Processor->OnAsyncCreatePhysicsState(Timeout);
		}
		case EJobType::DestroyPhysicsState:
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(FPhysScene_AsyncPhysicsStateJobQueue::DestroyPhysicsState);
			return Processor->OnAsyncDestroyPhysicsState(Timeout);
		}
		}
	}
	return true;
}

void FPhysScene_AsyncPhysicsStateJobQueue::FJob::OnPostExecute_GameThread() const
{
	if (IsValid()) 
	{
		switch (Type)
		{
			case EJobType::CreatePhysicsState:
			{
				Processor->OnAsyncCreatePhysicsStateEnd_GameThread();
			}
			break;
			case EJobType::DestroyPhysicsState:
			{
				Processor->OnAsyncDestroyPhysicsStateEnd_GameThread();
			}
			break;
		}
	}

	for (UObject* Obj : RootedInputs)
	{
		Owner->RemoveRootObject(Obj);
	}
	RootedInputs.Reset();
}