// Copyright Epic Games, Inc. All Rights Reserved.

#include "PhysicsEngine/ExternalSpatialAccelerationPayload.h"

#include "Components/PrimitiveComponent.h"
#include "PhysicsEngine/PhysicsObjectExternalInterface.h"

#if CHAOS_DEBUG_DRAW
#include "Chaos/ChaosDebugDraw.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(ExternalSpatialAccelerationPayload)

FExternalSpatialAccelerationPayload::FExternalSpatialAccelerationPayload()
{
}

void FExternalSpatialAccelerationPayload::Initialize(TObjectKey<UPrimitiveComponent> InComponent, int32 InBoneId)
{
	Component = InComponent;
	BoneId = InBoneId;

	if (Chaos::FGeometryParticle* Particle = GetExternalGeometryParticle_ExternalThread())
	{
		CachedUniqueIdx = Particle->UniqueIdx();
	}
}


void FExternalSpatialAccelerationPayload::Initialize(TObjectKey<UPrimitiveComponent> InComponent, int32 InBoneId, const Chaos::FUniqueIdx& UniqueIdx)
{
	Component = InComponent;
	BoneId = InBoneId;
	CachedUniqueIdx = UniqueIdx;
}


Chaos::FGeometryParticle* FExternalSpatialAccelerationPayload::GetExternalGeometryParticle_ExternalThread() const
{
	if (UPrimitiveComponent* ValidComponent = Component.ResolveObjectPtr())
	{
		if (!ValidComponent->HasValidPhysicsState())
		{
			return nullptr;
		}

		Chaos::FPhysicsObjectHandle Handle = ValidComponent->GetPhysicsObjectById(BoneId);
		FLockedReadPhysicsObjectExternalInterface Interface = FPhysicsObjectExternalInterface::LockRead(Handle);
		return Interface->GetParticle(Handle);
	}
	
	return nullptr;
}

Chaos::FUniqueIdx FExternalSpatialAccelerationPayload::UniqueIdx() const
{
	return CachedUniqueIdx;
}

void FExternalSpatialAccelerationPayload::Serialize(Chaos::FChaosArchive& Ar)
{
	// TODO:
}

#if CHAOS_DEBUG_DRAW
void FExternalSpatialAccelerationPayload::DebugDraw(const bool bExternal, const bool bHit) const
{
	if (bExternal)
	{
		if (Chaos::FGeometryParticle* ExternalGeometryParticle = GetExternalGeometryParticle_ExternalThread())
		{
			Chaos::DebugDraw::DrawParticleShapes(Chaos::FRigidTransform3(), ExternalGeometryParticle, bHit ? FColor::Red : FColor::Green);
		}
	}
}
#endif

uint32 GetTypeHash(const FExternalSpatialAccelerationPayload& Payload)
{
	const uint32 ComponentHash = GetTypeHash(Payload.Component);
	const uint32 BoneIdHash = GetTypeHash(Payload.BoneId);
	return HashCombine(ComponentHash, BoneIdHash);
}
