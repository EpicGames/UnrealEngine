// Copyright Epic Games, Inc. All Rights Reserved.
#include "Physics/ImmediatePhysics/ImmediatePhysicsAdapters.h"

#include "BodySetupEnums.h"
#include "Chaos/ChaosConstraintSettings.h"
#include "Chaos/MassProperties.h"
#include "Chaos/PBDRigidsSOAs.h"
#include "Chaos/Particle/ParticleUtilities.h"
#include "Chaos/TriangleMeshImplicitObject.h"
#include "PhysicalMaterials/PhysicalMaterial.h"
#include "Physics/Experimental/ChaosInterfaceUtils.h"
#include "Physics/ImmediatePhysics/ImmediatePhysicsActorHandle.h"
#include "Physics/ImmediatePhysics/ImmediatePhysicsJointHandle.h"
#include "Physics/PhysicsInterfaceTypes.h"
#include "PhysicsEngine/BodyInstance.h"
#include "PhysicsEngine/BodySetup.h"
#include "PhysicsEngine/BodyUtils.h"
#include "PhysicsEngine/ConstraintInstance.h"
#include "PhysicsProxy/SingleParticlePhysicsProxy.h"

int32 RBANForceJointSolverType = -1;
FAutoConsoleVariableRef CVarRigidBodyNodeForceJointSolverType(
	TEXT("p.RigidBodyNode.ForceJointSolverType"), 
	RBANForceJointSolverType, 
	TEXT("-1: RBAN joints use solver specific in settings; 0: All RBAN joints use the linear solver. 1: All RBAN joints use non-linear solver"), 
	ECVF_Default);

namespace ImmediatePhysics
{
	bool CreateDefaultGeometry(const FVector& Scale, Chaos::FReal& OutMass, Chaos::FVec3& OutInertia, Chaos::FRigidTransform3& OutCoMTransform, Chaos::FImplicitObjectPtr& OutGeom, TArray<TUniquePtr<Chaos::FPerShapeData>>& OutShapes)
	{
		using namespace Chaos;

		const FReal Mass = 1.0f;
		const FReal Radius = 1.0f * Scale.GetMax();

		Chaos::FImplicitObjectPtr ImplicitSphere = MakeImplicitObjectPtr<Chaos::FImplicitSphere3>(FVec3(0), Radius);
		TUniquePtr<FPerShapeData> NewShape = Chaos::FShapeInstance::Make(OutShapes.Num(), ImplicitSphere);
		NewShape->UpdateShapeBounds(FTransform::Identity);
		NewShape->SetUserData(nullptr);
		NewShape->SetQueryEnabled(false);
		NewShape->SetSimEnabled(false);

		OutMass = Mass;
		OutInertia = Chaos::FSphere::GetInertiaTensor(Mass, Radius).GetDiagonal();
		OutCoMTransform = FTransform::Identity;
		OutShapes.Emplace(MoveTemp(NewShape));
		OutGeom = MoveTemp(ImplicitSphere);

		return true;
	}

	// Clone wrapper geometry if it is wrapped in Instanced or Transformed, without cloning the inner shape. This is used to instance
	// world geometry like heightfields and trimeshes into the immediate physics simulation.
	Chaos::FImplicitObjectPtr CloneWrappedGeometry(const Chaos::FImplicitObject* Geom, TArray<TUniquePtr<Chaos::FPerShapeData>>& OutShapes)
	{
		using namespace Chaos;

		EImplicitObjectType GeomType = GetInnerType(Geom->GetCollisionType());
		bool bIsInstanced = IsInstanced(Geom->GetCollisionType());
		bool bIsScaled = IsScaled(Geom->GetCollisionType());

		// Transformed HeightField
		if (GeomType == ImplicitObjectType::Transformed)
		{
			const TImplicitObjectTransformed<FReal, 3>* SrcTransformed = Geom->template GetObject<const TImplicitObjectTransformed<FReal, 3>>();
			if ((SrcTransformed != nullptr) && (SrcTransformed->GetTransformedObject()->GetType() == ImplicitObjectType::HeightField))
			{
				FImplicitObject* InnerGeom = const_cast<FImplicitObject*>(SrcTransformed->GetTransformedObject());
				return MakeImplicitObjectPtr<Chaos::TImplicitObjectTransformed<FReal, 3, false>>(InnerGeom, SrcTransformed->GetTransform());
			}
		}

		// Instanced Trimesh
		if (bIsInstanced && (GeomType == ImplicitObjectType::TriangleMesh))
		{
			const TImplicitObjectInstanced<FTriangleMeshImplicitObject>* SrcInstanced = Geom->template GetObject<const TImplicitObjectInstanced<FTriangleMeshImplicitObject>>();
			if (SrcInstanced != nullptr)
			{
				const TImplicitObjectInstanced<FTriangleMeshImplicitObject>::ObjectType InnerGeom = SrcInstanced->Object();
				return  MakeImplicitObjectPtr<TImplicitObjectInstanced<FTriangleMeshImplicitObject>>(InnerGeom);
			}
		}

		return nullptr;
	}

	// Intended for use with Tri Mesh and Heightfields when cloning world simulation objects into the immediate scene
	bool CloneWrappedGeometry(const FBodyInstance* BodyInstance, EActorType ActorType, const FVector& Scale, Chaos::FReal& OutMass, Chaos::FVec3& OutInertia, Chaos::FRigidTransform3& OutCoMTransform, Chaos::FImplicitObjectPtr& OutGeom, TArray<TUniquePtr<Chaos::FPerShapeData>>& OutShapes)
	{
		// We should only get non-simulated objects through this path, but you never know...
		if ((BodyInstance != nullptr) && !BodyInstance->bSimulatePhysics && BodyInstance->GetPhysicsActor())
		{
			OutMass = 0.0f;
			OutInertia = FVector::ZeroVector;
			OutCoMTransform = FTransform::Identity;
			OutGeom = CloneWrappedGeometry(BodyInstance->GetPhysicsActor()->GetGameThreadAPI().GetGeometry(), OutShapes);
			if (OutGeom != nullptr)
			{
				return true;
			}
		}

		return CreateDefaultGeometry(Scale, OutMass, OutInertia, OutCoMTransform, OutGeom, OutShapes);
	}

	bool CreateGeometry(const FBodyInstance* BodyInstance, EActorType ActorType, const FVector& Scale, Chaos::FReal& OutMass, Chaos::FVec3& OutInertia, Chaos::FRigidTransform3& OutCoMTransform, Chaos::FImplicitObjectPtr& OutGeom, TArray<TUniquePtr<Chaos::FPerShapeData>>& OutShapes)
	{
		using namespace Chaos;

		OutMass = 0.0f;
		OutInertia = FVector::ZeroVector;
		OutCoMTransform = FTransform::Identity;

		// If there's no BodySetup, we may be cloning an in-world object and probably have a TriMesh or HeightField so try to just copy references
		// @todo(ccaulfield): make this cleaner - we should have a separate path for this
		if ((BodyInstance == nullptr) || (BodyInstance->BodySetup == nullptr) || (BodyInstance->BodySetup->CollisionTraceFlag == ECollisionTraceFlag::CTF_UseComplexAsSimple))
		{
			return CloneWrappedGeometry(BodyInstance, ActorType, Scale, OutMass, OutInertia, OutCoMTransform, OutGeom, OutShapes);
		}

		UBodySetup* BodySetup = BodyInstance->GetBodySetup();

		// Set the filter to collide with everything (we use a broad phase that only contains particle pairs that are explicitly set to collide)
		FBodyCollisionData BodyCollisionData;

		const Chaos::Filter::FShapeFilterData ShapeFilterData = Chaos::Filter::FShapeFilterBuilder::BuildBlockAll(EFilterFlags::All);
		BodyCollisionData.CollisionFilterData.SimpleShapeFilterData = ShapeFilterData;
		BodyCollisionData.CollisionFilterData.ComplexShapeFilterData = ShapeFilterData;
		BodyCollisionData.CollisionFlags.bEnableSimCollisionSimple = true;
		BodyCollisionData.CollisionFlags.bEnableQueryCollision = false;

		BodyInstance->BuildBodyCollisionFlags(BodyCollisionData.CollisionFlags, BodyInstance->GetCollisionEnabled(), BodyInstance->BodySetup->GetCollisionTraceFlag() == CTF_UseComplexAsSimple);

		FGeometryAddParams AddParams;
		AddParams.bDoubleSided = BodySetup->bDoubleSidedGeometry;
		AddParams.CollisionData = BodyCollisionData;
		AddParams.CollisionTraceType = BodySetup->GetCollisionTraceFlag();
		AddParams.Scale = Scale;
		//AddParams.SimpleMaterial = SimpleMaterial;
		//AddParams.ComplexMaterials = TArrayView<UPhysicalMaterial*>(ComplexMaterials);
		AddParams.LocalTransform = FTransform::Identity;
		AddParams.WorldTransform = BodyInstance->GetUnrealWorldTransform();
		AddParams.Geometry = &BodySetup->AggGeom;
		AddParams.TriMeshGeometries = MakeArrayView(BodySetup->TriMeshGeometries);

		TArray<Chaos::FImplicitObjectPtr> Geoms;
		FShapesArray Shapes;
		ChaosInterface::CreateGeometry(AddParams, Geoms, Shapes);

		if (Geoms.Num() == 0)
		{
			return false;
		}

		if (ActorType == EActorType::DynamicActor)
		{
			// Whether each shape contributes to mass
			// @todo(chaos): it would be easier if ComputeMassProperties knew how to extract this info. Maybe it should be a flag in PerShapeData
			TArray<bool> bContributesToMass;
			bContributesToMass.Reserve(Shapes.Num());
			for (int32 ShapeIndex = 0; ShapeIndex < Shapes.Num(); ++ShapeIndex)
			{
				const TUniquePtr<FPerShapeData>& Shape = Shapes[ShapeIndex];
				const FKShapeElem* ShapeElem = FChaosUserData::Get<FKShapeElem>(Shape->GetUserData());
				bool bHasMass = ShapeElem && ShapeElem->GetContributeToMass();
				bContributesToMass.Add(bHasMass);
			}

			// bInertaScaleIncludeMass = true is to match legacy physics behaviour. This will scale the inertia by the change in mass (density x volumescale) 
			// as well as the dimension change even though we don't actually change the mass.
			const bool bInertaScaleIncludeMass = true;
			FMassProperties MassProperties = BodyUtils::ComputeMassProperties(BodyInstance, Shapes, bContributesToMass, FTransform::Identity, bInertaScaleIncludeMass);
			OutMass = MassProperties.Mass;
			OutInertia = MassProperties.InertiaTensor.GetDiagonal();
			OutCoMTransform = FTransform(MassProperties.RotationOfMass, MassProperties.CenterOfMass);
		}

		// If we have multiple root shapes, wrap them in a union
		if (Geoms.Num() == 1)
		{
			OutGeom = MoveTemp(Geoms[0]);
		}
		else
		{
			OutGeom = MakeImplicitObjectPtr<FImplicitObjectUnion>(MoveTemp(Geoms));
		}

		for (TUniquePtr<FPerShapeData>& Shape : Shapes)
		{
			OutShapes.Emplace(MoveTemp(Shape));
		}

		return true;
	}

	FActorSetup MakeStaticActorSetup(const FBodyInstance* InBodyInstance, const FTransform& InTransform)
	{
		return MakeActorSetup(EActorType::StaticActor, InBodyInstance, InTransform);
	}

	FActorSetup MakeKinematicActorSetup(const FBodyInstance* InBodyInstance, const FTransform& InTransform)
	{
		return MakeActorSetup(EActorType::KinematicActor, InBodyInstance, InTransform);
	}

	FActorSetup MakeDynamicActorSetup(const FBodyInstance* InBodyInstance, const FTransform& InTransform)
	{
		return MakeActorSetup(EActorType::DynamicActor, InBodyInstance, InTransform);
	}

	FActorSetup MakeActorSetup(EActorType InActorType, const FBodyInstance* InBodyInstance, const FTransform& InTransform)
	{
		FActorSetup ActorSetup;
		ActorSetup.ActorType = InActorType;
		ActorSetup.Transform = InTransform;

		Chaos::FReal BodyMass = 0.0f;
		Chaos::FVec3 BodyInertia = Chaos::FVec3::OneVector;
		Chaos::FRigidTransform3 BodyCoMTransform = Chaos::FRigidTransform3::Identity;
		Chaos::FImplicitObjectPtr BodyGeom;
		TArray<TUniquePtr<Chaos::FPerShapeData>> BodyShapes;
		bool bHasGeometry = CreateGeometry(InBodyInstance, InActorType, InTransform.GetScale3D(), BodyMass, BodyInertia, BodyCoMTransform, BodyGeom, BodyShapes);
		if (!bHasGeometry)
		{
			return MoveTemp(ActorSetup);
		}

		ActorSetup.Geometry = MoveTemp(BodyGeom);
		ActorSetup.Shapes = MoveTemp(BodyShapes);
		ActorSetup.Material = MakeUnique<Chaos::FChaosPhysicsMaterial>();

		ActorSetup.Mass = BodyMass;
		ActorSetup.Inertia = BodyInertia;
		ActorSetup.CoMTransform = BodyCoMTransform;

		if (InBodyInstance != nullptr)
		{
			ActorSetup.LinearDamping = InBodyInstance->LinearDamping;
			ActorSetup.AngularDamping = InBodyInstance->AngularDamping;

			ActorSetup.CollisionEnabled = InBodyInstance->GetCollisionEnabled();
			ActorSetup.GravityGroupIndex = InBodyInstance->GravityGroupIndex;
			ActorSetup.bInertiaConditioningEnabled = InBodyInstance->IsInertiaConditioningEnabled();
			ActorSetup.bEnableGravity = InBodyInstance->bEnableGravity;
			ActorSetup.bUpdateKinematicFromSimulation = InBodyInstance->bUpdateKinematicFromSimulation;
			ActorSetup.bGyroscopicTorqueEnabled = InBodyInstance->bGyroscopicTorqueEnabled;

			// @todo(ccaulfield): We cannot ask for the physical material on a task thread, because FMICReentranceGuard in UMaterialInstance will assert (in editor). Fix this...
			// For now we just use material defaults when actors are created on a task thread. This happens when adding world-objects to a RigidBody AnimNode simulation.
			if (IsInGameThread())
			{
				if (UPhysicalMaterial* SimplePhysMat = InBodyInstance->GetSimplePhysicalMaterial())
				{
					ActorSetup.Material->Friction = SimplePhysMat->Friction;
					ActorSetup.Material->Restitution = SimplePhysMat->Restitution;
				}
			}
		}

		return MoveTemp(ActorSetup);
	}

	static_assert((int32)Chaos::EJointMotionType::Free == (int32)EAngularConstraintMotion::ACM_Free, "Chaos::EJointMotionType and EAngularConstraintMotion mismatch");
	static_assert((int32)Chaos::EJointMotionType::Limited == (int32)EAngularConstraintMotion::ACM_Limited, "Chaos::EJointMotionType and EAngularConstraintMotion mismatch");
	static_assert((int32)Chaos::EJointMotionType::Locked == (int32)EAngularConstraintMotion::ACM_Locked, "Chaos::EJointMotionType and EAngularConstraintMotion mismatch");

	// NOTE: Hard dependence on EJointAngularConstraintIndex - the following will break if we change the order (but can be easily fixed). See FJointHandle::FJointHandle
	static_assert((int32)Chaos::EJointAngularConstraintIndex::Twist == 0, "Angular drive targets have hard dependency on constraint order");
	static_assert((int32)Chaos::EJointAngularConstraintIndex::Swing1 == 2, "Angular drive targets have hard dependency on constraint order");

	static Chaos::EJointMotionType ConvertToJointMotionType(ELinearConstraintMotion InType)
	{
		switch (InType)
		{
		case ELinearConstraintMotion::LCM_Free: return Chaos::EJointMotionType::Free;
		case ELinearConstraintMotion::LCM_Limited: return Chaos::EJointMotionType::Limited;
		case ELinearConstraintMotion::LCM_Locked: return Chaos::EJointMotionType::Locked;
		default: ensure(false); return Chaos::EJointMotionType::Locked;
		}
	};

	static Chaos::EJointMotionType ConvertToJointMotionType(EAngularConstraintMotion InType)
	{
		switch (InType)
		{
		case EAngularConstraintMotion::ACM_Free: return Chaos::EJointMotionType::Free;
		case EAngularConstraintMotion::ACM_Limited: return Chaos::EJointMotionType::Limited;
		case EAngularConstraintMotion::ACM_Locked: return Chaos::EJointMotionType::Locked;
		default: ensure(false); return Chaos::EJointMotionType::Locked;
		}
	};

	static Chaos::EPlasticityType ConvertToPlasticityType(EConstraintPlasticityType InType)
	{
		switch (InType)
		{
		case EConstraintPlasticityType::CCPT_Free: return Chaos::EPlasticityType::Free;
		case EConstraintPlasticityType::CCPT_Shrink: return Chaos::EPlasticityType::Shrink;
		case EConstraintPlasticityType::CCPT_Grow: return Chaos::EPlasticityType::Grow;
		default: return Chaos::EPlasticityType::Free;
		}
	}

	void UpdateJointSettingsFromLinearConstraint(const FLinearConstraint& Constraint, Chaos::FPBDJointSettings& JointSettings)
	{
		using namespace Chaos;

		JointSettings.LinearMotionTypes[0] = ConvertToJointMotionType(Constraint.XMotion);
		JointSettings.LinearMotionTypes[1] = ConvertToJointMotionType(Constraint.YMotion);
		JointSettings.LinearMotionTypes[2] = ConvertToJointMotionType(Constraint.ZMotion);

		JointSettings.LinearLimit = Constraint.Limit;
		JointSettings.bSoftLinearLimitsEnabled = Constraint.bSoftConstraint;

		JointSettings.SoftLinearStiffness =
			Chaos::ConstraintSettings::SoftLinearStiffnessScale() * Constraint.Stiffness;
		JointSettings.SoftLinearDamping =
			Chaos::ConstraintSettings::SoftLinearDampingScale() * Constraint.Damping;
		JointSettings.LinearRestitution = Constraint.Restitution;
		JointSettings.LinearContactDistance = Constraint.ContactDistance;

		// UE Disables Soft Limits when the Limit is less than some threshold. This is not necessary
		// in Chaos but for now we also do it for parity's sake (See FLinearConstraint::UpdateLinearLimit_AssumesLocked).
		if (JointSettings.LinearLimit < RB_MinSizeToLockDOF)
		{
			for (int32 AxisIndex = 0; AxisIndex < 3; ++AxisIndex)
			{
				if (JointSettings.LinearMotionTypes[AxisIndex] == EJointMotionType::Limited)
				{
					JointSettings.LinearMotionTypes[AxisIndex] = EJointMotionType::Locked;
				}
			}
		}
	}

	// twist/swing Order is twist, swing1, swing2 and in degrees

	void UpdateJointSettingsFromConeConstraint(const FConeConstraint& Constraint, Chaos::FPBDJointSettings& JointSettings)
	{
		JointSettings.AngularMotionTypes[(int32)Chaos::EJointAngularConstraintIndex::Swing1] = ConvertToJointMotionType(Constraint.Swing1Motion);
		JointSettings.AngularMotionTypes[(int32)Chaos::EJointAngularConstraintIndex::Swing2] = ConvertToJointMotionType(Constraint.Swing2Motion);

		JointSettings.AngularLimits[(int32)Chaos::EJointAngularConstraintIndex::Swing1] = FMath::DegreesToRadians(Constraint.Swing1LimitDegrees);
		JointSettings.AngularLimits[(int32)Chaos::EJointAngularConstraintIndex::Swing2] = FMath::DegreesToRadians(Constraint.Swing2LimitDegrees);

		JointSettings.bSoftSwingLimitsEnabled = Constraint.bSoftConstraint;

		JointSettings.SoftSwingStiffness =
			Chaos::ConstraintSettings::SoftAngularStiffnessScale() * Constraint.Stiffness;
		JointSettings.SoftSwingDamping =
			Chaos::ConstraintSettings::SoftAngularDampingScale() * Constraint.Damping;

		JointSettings.SwingRestitution = Constraint.Restitution;

		JointSettings.SwingContactDistance = Constraint.ContactDistance;
	}

	void UpdateJointSettingsFromTwistConstraint(const FTwistConstraint& Constraint, Chaos::FPBDJointSettings& JointSettings)
	{
		JointSettings.AngularMotionTypes[(int32)Chaos::EJointAngularConstraintIndex::Twist] = ConvertToJointMotionType(Constraint.TwistMotion);
		JointSettings.AngularLimits[(int32)Chaos::EJointAngularConstraintIndex::Twist] = FMath::DegreesToRadians(Constraint.TwistLimitDegrees);
		JointSettings.bSoftTwistLimitsEnabled = Constraint.bSoftConstraint;
		JointSettings.SoftTwistStiffness =
			Chaos::ConstraintSettings::SoftAngularStiffnessScale() * Constraint.Stiffness;
		JointSettings.SoftTwistDamping =
			Chaos::ConstraintSettings::SoftAngularDampingScale() * Constraint.Damping;
		JointSettings.TwistRestitution = Constraint.Restitution;
		JointSettings.TwistContactDistance = Constraint.ContactDistance;
	}

	// Copies data from a profile into settings
	void UpdateJointSettingsFromConstraintProfile(const FConstraintProfileProperties& Profile, Chaos::FPBDJointSettings& JointSettings)
	{
		using namespace Chaos;

		JointSettings.Stiffness = ConstraintSettings::JointStiffness();
		JointSettings.LinearProjection = Profile.bEnableProjection ? Profile.ProjectionLinearAlpha : 0.0f;
		JointSettings.AngularProjection = Profile.bEnableProjection ? Profile.ProjectionAngularAlpha : 0.0f;
		JointSettings.ShockPropagation = Profile.bEnableShockPropagation ? Profile.ShockPropagationAlpha : 0.0f;
		JointSettings.TeleportDistance = Profile.bEnableProjection ? Profile.ProjectionLinearTolerance : -1.0f;
		JointSettings.TeleportAngle = Profile.bEnableProjection ?
			FMath::DegreesToRadians(Profile.ProjectionAngularTolerance) : -1.0f;
		JointSettings.ParentInvMassScale = Profile.bParentDominates ? (FReal)0 : (FReal)1;

		JointSettings.bCollisionEnabled = !Profile.bDisableCollision;
		JointSettings.bProjectionEnabled = Profile.bEnableProjection;
		JointSettings.bShockPropagationEnabled = Profile.bEnableShockPropagation;
		JointSettings.bMassConditioningEnabled = Profile.bEnableMassConditioning;

		JointSettings.bUseLinearSolver = Profile.bUseLinearJointSolver;
		if (RBANForceJointSolverType >= 0)
		{
			JointSettings.bUseLinearSolver = (RBANForceJointSolverType == 0);
		}

		UpdateJointSettingsFromLinearConstraint(Profile.LinearLimit, JointSettings);
		UpdateJointSettingsFromConeConstraint(Profile.ConeLimit, JointSettings);
		UpdateJointSettingsFromTwistConstraint(Profile.TwistLimit, JointSettings);

		JointSettings.LinearSoftForceMode = (ConstraintSettings::SoftLinearForceMode() == 0) ?
			EJointForceMode::Acceleration : EJointForceMode::Force;
		JointSettings.AngularSoftForceMode = (ConstraintSettings::SoftAngularForceMode() == 0) ?
			EJointForceMode::Acceleration : EJointForceMode::Force;

		UpdateJointSettingsFromLinearDriveConstraint(Profile.LinearDrive, JointSettings);
		UpdateJointSettingsFromAngularDriveConstraint(Profile.AngularDrive, JointSettings);

		JointSettings.LinearBreakForce = Profile.bLinearBreakable ?
			ConstraintSettings::LinearBreakScale() * Profile.LinearBreakThreshold : FLT_MAX;
		JointSettings.LinearPlasticityLimit = Profile.bLinearPlasticity ?
			FMath::Clamp((float)Profile.LinearPlasticityThreshold, 0.0f, 1.0f) : FLT_MAX;
		JointSettings.LinearPlasticityType = ConvertToPlasticityType(Profile.LinearPlasticityType);
		// JointSettings.LinearPlasticityInitialDistanceSquared = ; // What do we do with this?

		JointSettings.AngularBreakTorque = Profile.bAngularBreakable ?
			ConstraintSettings::AngularBreakScale() * Profile.AngularBreakThreshold : FLT_MAX;
		JointSettings.AngularPlasticityLimit = Profile.bAngularPlasticity ?
			FMath::Clamp((float)Profile.AngularPlasticityThreshold, 0.0f, 1.0f) : FLT_MAX;;

		JointSettings.ContactTransferScale = Profile.ContactTransferScale;
	}


	FJointSetup MakeJointSetup(const FConstraintInstance* InConstraintInstance, FActorHandle* InActor1, FActorHandle* InActor2)
	{
		using namespace Chaos;

		FJointSetup JointSetup;
		JointSetup.Actor1 = InActor1;
		JointSetup.Actor2 = InActor2;

		if (InConstraintInstance != nullptr)
		{
			UpdateJointSettingsFromConstraintProfile(InConstraintInstance->ProfileInstance, JointSetup.ConstraintSettings);
			FReal JointScale = InConstraintInstance->GetLastKnownScale();
			JointSetup.ConstraintSettings.ConnectorTransforms[0] = InConstraintInstance->GetRefFrame(EConstraintFrame::Frame1);
			JointSetup.ConstraintSettings.ConnectorTransforms[1] = InConstraintInstance->GetRefFrame(EConstraintFrame::Frame2);
			JointSetup.ConstraintSettings.ConnectorTransforms[0].ScaleTranslation(JointScale);
			JointSetup.ConstraintSettings.ConnectorTransforms[1].ScaleTranslation(JointScale);

			JointSetup.ConstraintSettings.Sanitize();
		}
		else
		{
			// TEMP: all creation with null ConstraintIndex for PhAT handles
			FTransform Actor1Transform = (InActor1 != nullptr) ? InActor1->GetWorldTransform() : FTransform::Identity;
			FTransform Actor2Transform = (InActor2 != nullptr) ? InActor2->GetWorldTransform() : FTransform::Identity;
			JointSetup.ConstraintSettings.ConnectorTransforms[0] = Actor2Transform.GetRelativeTransform(Actor1Transform);
			JointSetup.ConstraintSettings.ConnectorTransforms[1] = FRigidTransform3();
			JointSetup.ConstraintSettings.LinearMotionTypes = { EJointMotionType::Limited, EJointMotionType::Limited, EJointMotionType::Limited };
			JointSetup.ConstraintSettings.LinearLimit = 0.1f;
			JointSetup.ConstraintSettings.SoftLinearStiffness = 500.0f;
			JointSetup.ConstraintSettings.SoftLinearDamping = 100.0f;
			JointSetup.ConstraintSettings.bSoftLinearLimitsEnabled = true;
			JointSetup.ConstraintSettings.LinearSoftForceMode = EJointForceMode::Acceleration;
			JointSetup.ConstraintSettings.LinearProjection = 0.0f;
			JointSetup.ConstraintSettings.AngularProjection = 0.0f;
			JointSetup.ConstraintSettings.TeleportDistance = -1.0f;
			JointSetup.ConstraintSettings.TeleportAngle = -1.0f;
		}

		return JointSetup;
	}
	
	void UpdateJointSettingsFromLinearDriveConstraint(const FLinearDriveConstraint& LinearDriveConstraint, Chaos::FPBDJointSettings& JointSettings)
	{
		using namespace Chaos;

		JointSettings.LinearDrivePositionTarget = LinearDriveConstraint.PositionTarget;
		JointSettings.LinearDriveVelocityTarget = LinearDriveConstraint.VelocityTarget;
		JointSettings.bLinearPositionDriveEnabled[0] = LinearDriveConstraint.XDrive.bEnablePositionDrive;
		JointSettings.bLinearPositionDriveEnabled[1] = LinearDriveConstraint.YDrive.bEnablePositionDrive;
		JointSettings.bLinearPositionDriveEnabled[2] = LinearDriveConstraint.ZDrive.bEnablePositionDrive;
		JointSettings.bLinearVelocityDriveEnabled[0] = LinearDriveConstraint.XDrive.bEnableVelocityDrive;
		JointSettings.bLinearVelocityDriveEnabled[1] = LinearDriveConstraint.YDrive.bEnableVelocityDrive;
		JointSettings.bLinearVelocityDriveEnabled[2] = LinearDriveConstraint.ZDrive.bEnableVelocityDrive;

		JointSettings.LinearDriveForceMode = EJointForceMode::Acceleration; // hardcoded!
		JointSettings.LinearDriveStiffness = Chaos::ConstraintSettings::LinearDriveStiffnessScale() * FVec3(
			LinearDriveConstraint.XDrive.Stiffness,
			LinearDriveConstraint.YDrive.Stiffness,
			LinearDriveConstraint.ZDrive.Stiffness);
		JointSettings.LinearDriveDamping = Chaos::ConstraintSettings::LinearDriveDampingScale() * FVec3(
			LinearDriveConstraint.XDrive.Damping,
			LinearDriveConstraint.YDrive.Damping,
			LinearDriveConstraint.ZDrive.Damping);
		JointSettings.LinearDriveMaxForce[0] = LinearDriveConstraint.XDrive.MaxForce;
		JointSettings.LinearDriveMaxForce[1] = LinearDriveConstraint.YDrive.MaxForce;
		JointSettings.LinearDriveMaxForce[2] = LinearDriveConstraint.ZDrive.MaxForce;
	}

	void UpdateJointSettingsFromAngularDriveConstraint(
		const FAngularDriveConstraint& AngularDriveConstraint, Chaos::FPBDJointSettings& JointSettings)
	{
		using namespace Chaos;

		JointSettings.AngularDrivePositionTarget = FQuat(AngularDriveConstraint.OrientationTarget);
		JointSettings.AngularDriveVelocityTarget = AngularDriveConstraint.AngularVelocityTarget * UE_TWO_PI; // rev/s to rad/s

		JointSettings.AngularDriveForceMode = EJointForceMode::Acceleration; // hardcoded!
		if (AngularDriveConstraint.AngularDriveMode == EAngularDriveMode::SLERP)
		{
			JointSettings.AngularDriveStiffness = FVec3(
				ConstraintSettings::AngularDriveStiffnessScale() * AngularDriveConstraint.SlerpDrive.Stiffness);
			JointSettings.AngularDriveDamping = FVec3(
				ConstraintSettings::AngularDriveDampingScale() * AngularDriveConstraint.SlerpDrive.Damping);
			JointSettings.AngularDriveMaxTorque = FVec3(
				AngularDriveConstraint.SlerpDrive.MaxForce);
			JointSettings.bAngularSLerpPositionDriveEnabled = AngularDriveConstraint.SlerpDrive.bEnablePositionDrive;
			JointSettings.bAngularSLerpVelocityDriveEnabled = AngularDriveConstraint.SlerpDrive.bEnableVelocityDrive;
			JointSettings.bAngularTwistPositionDriveEnabled = false;
			JointSettings.bAngularTwistVelocityDriveEnabled = false;
			JointSettings.bAngularSwingPositionDriveEnabled = false;
			JointSettings.bAngularSwingVelocityDriveEnabled = false;
		}
		else
		{
			JointSettings.AngularDriveStiffness = ConstraintSettings::AngularDriveStiffnessScale() * FVec3(
				AngularDriveConstraint.TwistDrive.Stiffness,
				AngularDriveConstraint.SwingDrive.Stiffness,
				AngularDriveConstraint.SwingDrive.Stiffness);
			JointSettings.AngularDriveDamping = ConstraintSettings::AngularDriveDampingScale() * FVec3(
				AngularDriveConstraint.TwistDrive.Damping,
				AngularDriveConstraint.SwingDrive.Damping,
				AngularDriveConstraint.SwingDrive.Damping);
			JointSettings.AngularDriveMaxTorque[0] = AngularDriveConstraint.TwistDrive.MaxForce;
			JointSettings.AngularDriveMaxTorque[1] = AngularDriveConstraint.SwingDrive.MaxForce;
			JointSettings.AngularDriveMaxTorque[2] = AngularDriveConstraint.SwingDrive.MaxForce;
			JointSettings.bAngularSLerpPositionDriveEnabled = false;
			JointSettings.bAngularSLerpVelocityDriveEnabled = false;
			JointSettings.bAngularTwistPositionDriveEnabled = AngularDriveConstraint.TwistDrive.bEnablePositionDrive;
			JointSettings.bAngularTwistVelocityDriveEnabled = AngularDriveConstraint.TwistDrive.bEnableVelocityDrive;
			JointSettings.bAngularSwingPositionDriveEnabled = AngularDriveConstraint.SwingDrive.bEnablePositionDrive;
			JointSettings.bAngularSwingVelocityDriveEnabled = AngularDriveConstraint.SwingDrive.bEnableVelocityDrive;
		}
	}
}

