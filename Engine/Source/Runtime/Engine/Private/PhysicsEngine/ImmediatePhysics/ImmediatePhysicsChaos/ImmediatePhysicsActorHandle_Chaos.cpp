// Copyright Epic Games, Inc. All Rights Reserved.

#include "Physics/ImmediatePhysics/ImmediatePhysicsChaos/ImmediatePhysicsActorHandle_Chaos.h"

#include "Chaos/PBDRigidsSOAs.h"
#include "Chaos/Particle/ParticleUtilities.h"
#include "Physics/ImmediatePhysics/ImmediatePhysicsShared/ImmediatePhysicsCore.h"
#include "PhysicsProxy/SingleParticlePhysicsProxy.h"

namespace ImmediatePhysics_Chaos
{
	//
	// FActorSetup
	//

	FActorSetup::FActorSetup()
		: Transform(FTransform::Identity)
		, Geometry{}
		, Shapes{}
		, Material{}
		, Mass(1.0)
		, Inertia(FVector::OneVector)
		, CoMTransform(FTransform::Identity)
		, LinearDamping(0.0)
		, AngularDamping(0.0)
		, ActorType(EActorType::KinematicActor)
		, CollisionEnabled(ECollisionEnabled::PhysicsOnly)
		, GravityGroupIndex(0)
		, bInertiaConditioningEnabled(true)
		, bEnableGravity(true)
		, bUpdateKinematicFromSimulation(true)
		, bGyroscopicTorqueEnabled(false)
	{
	}

	FActorSetup::FActorSetup(FActorSetup&& R)
		: Transform(R.Transform)
		, Geometry(R.Geometry)
		, Shapes(MoveTemp(R.Shapes))
		, Material(MoveTemp(R.Material))
		, Mass(R.Mass)
		, Inertia(R.Inertia)
		, CoMTransform(R.CoMTransform)
		, LinearDamping(R.LinearDamping)
		, AngularDamping(R.AngularDamping)
		, ActorType(R.ActorType)
		, CollisionEnabled(R.CollisionEnabled)
		, GravityGroupIndex(R.GravityGroupIndex)
		, bInertiaConditioningEnabled(R.bInertiaConditioningEnabled)
		, bEnableGravity(R.bEnableGravity)
		, bUpdateKinematicFromSimulation(R.bUpdateKinematicFromSimulation)
		, bGyroscopicTorqueEnabled(R.bGyroscopicTorqueEnabled)
	{
	}

	FActorSetup::~FActorSetup()
	{
	}

	//
	// FActorHandle
	//

	void FActorHandle::CreateParticleHandle(
		FActorSetup&& ActorSetup)
	{
		using namespace Chaos;

		switch (ActorSetup.ActorType)
		{
		case EActorType::StaticActor:
			ParticleHandle = Particles.CreateStaticParticles(1, nullptr, FGeometryParticleParameters())[0];
			break;
		case EActorType::KinematicActor:
			ParticleHandle = Particles.CreateKinematicParticles(1, nullptr, FKinematicGeometryParticleParameters())[0];
			break;
		case EActorType::DynamicActor:
			ParticleHandle = Particles.CreateDynamicParticles(1, nullptr, FPBDRigidParticleParameters())[0];
			break;
		}

		if (ParticleHandle != nullptr)
		{
			SetWorldTransform(ActorSetup.Transform);

			ParticleHandle->SetGeometry(ActorSetup.Geometry);

			// Set the collision filter data for the shapes to collide with everything.
			// Even though we already tried to do this when we created the original shapes array, 
			// that gets thrown away and we need to do it here. This is not a good API
			const Chaos::Filter::FShapeFilterData ShapeFilter = Chaos::Filter::FShapeFilterBuilder::BuildBlockAll(Chaos::EFilterFlags::All);

			FCollisionData CollisionData;
			CollisionData.SetShapeFilterData(ShapeFilter);
			CollisionData.bSimCollision = 1;
			CollisionData.bQueryCollision = 0;
			for (const TUniquePtr<FPerShapeData>& Shape : ParticleHandle->ShapesArray())
			{
				Shape->SetCollisionData(CollisionData);
			}

			if (ParticleHandle->GetGeometry() && ParticleHandle->GetGeometry()->HasBoundingBox())
			{
				ParticleHandle->SetHasBounds(true);
				ParticleHandle->SetLocalBounds(ParticleHandle->GetGeometry()->BoundingBox());
				ParticleHandle->UpdateWorldSpaceState(FRigidTransform3(ParticleHandle->GetX(), ParticleHandle->GetR()), FVec3(0));
			}

			if (FKinematicGeometryParticleHandle* Kinematic = ParticleHandle->CastToKinematicParticle())
			{
				Kinematic->SetVf(FVector3f::ZeroVector);
				Kinematic->SetWf(FVector3f::ZeroVector);
			}

			FPBDRigidParticleHandle* Dynamic = ParticleHandle->CastToRigidParticle();
			if (Dynamic && Dynamic->ObjectState() == EObjectStateType::Dynamic)
			{
				FReal MassInv = (ActorSetup.Mass > 0.0f) ? 1.0f / ActorSetup.Mass : 0.0f;
				FVec3 InertiaInv = (ActorSetup.Mass > 0.0f) ? ActorSetup.Inertia.Reciprocal() : FVec3::ZeroVector;
				Dynamic->SetM(ActorSetup.Mass);
				Dynamic->SetInvM(MassInv);
				Dynamic->SetCenterOfMass(ActorSetup.CoMTransform.GetTranslation());
				Dynamic->SetRotationOfMass(ActorSetup.CoMTransform.GetRotation());
				Dynamic->SetI(TVec3<FRealSingle>(ActorSetup.Inertia.X, ActorSetup.Inertia.Y, ActorSetup.Inertia.Z));
				Dynamic->SetInvI(TVec3<FRealSingle>(InertiaInv.X, InertiaInv.Y, InertiaInv.Z));
				Dynamic->SetInertiaConditioningEnabled(ActorSetup.bInertiaConditioningEnabled);
				Dynamic->SetLinearEtherDrag(ActorSetup.LinearDamping);
				Dynamic->SetAngularEtherDrag(ActorSetup.AngularDamping);
				Dynamic->SetGravityEnabled(ActorSetup.bEnableGravity);
				Dynamic->SetGravityGroupIndex(ActorSetup.GravityGroupIndex);
				Dynamic->SetUpdateKinematicFromSimulation(ActorSetup.bUpdateKinematicFromSimulation);
				Dynamic->SetGyroscopicTorqueEnabled(ActorSetup.bGyroscopicTorqueEnabled);
				Dynamic->SetDisabled(true);
			}
		}
	}

	//
	// Actor Handle
	//

	FActorHandle::FActorHandle(
		Chaos::FPBDRigidsSOAs& InParticles, 
		Chaos::TArrayCollectionArray<Chaos::FVec3>& InParticlePrevXs, 
		Chaos::TArrayCollectionArray<Chaos::FRotation3>& InParticlePrevRs, 
		FActorSetup&& ActorSetup)
		: Particles(InParticles)
		, ParticleHandle(nullptr)
		, ParticlePrevXs(InParticlePrevXs)
		, ParticlePrevRs(InParticlePrevRs)
	{
		CreateParticleHandle(MoveTemp(ActorSetup));
	}

	FActorHandle::~FActorHandle()
	{
		if (ParticleHandle != nullptr)
		{
			Particles.DestroyParticle(ParticleHandle);
			ParticleHandle = nullptr;
		}
	}

	Chaos::FGenericParticleHandle FActorHandle::Handle() const
	{
		return { ParticleHandle };
	}

	Chaos::FGeometryParticleHandle* FActorHandle::GetParticle()
	{
		return ParticleHandle;
	}

	const Chaos::FGeometryParticleHandle* FActorHandle::GetParticle() const
	{
		return ParticleHandle;
	}

	bool FActorHandle::GetEnabled() const
	{
		const Chaos::FConstGenericParticleHandle Particle = ParticleHandle;
		return !Particle->Disabled();
	}

	void FActorHandle::SetEnabled(bool bEnabled)
	{
		Chaos::FPBDRigidParticleHandle* Dynamic = ParticleHandle->CastToRigidParticle();
		if (Dynamic && Dynamic->ObjectState() == Chaos::EObjectStateType::Dynamic)
		{
			Dynamic->SetDisabled(!bEnabled);
		}
	}

	bool FActorHandle::GetHasCollision() const
	{
		return ParticleHandle->HasCollision();
	}

	void FActorHandle::SetHasCollision(bool bCollision)
	{
		ParticleHandle->SetHasCollision(bCollision);
	}

	void FActorHandle::InitWorldTransform(const FTransform& WorldTM)
	{
		using namespace Chaos;

		SetWorldTransform(WorldTM);

		if (FKinematicGeometryParticleHandle* Kinematic = ParticleHandle->CastToKinematicParticle())
		{
			Kinematic->SetVf(FVec3f(0));
			Kinematic->SetWf(FVec3f(0));
			Kinematic->KinematicTarget().Clear();
		}

		// Initialize the bounds. Important because if the particle never moves its 
		// bounds will never get updated (see FPBDMinEvolution::ApplyKinematicTargets) 
		ParticleHandle->UpdateWorldSpaceState(FRigidTransform3(ParticleHandle->GetX(), ParticleHandle->GetR()), FVec3(0));
	}

	void FActorHandle::SetWorldTransform(const FTransform& WorldTM)
	{
		using namespace Chaos;

		ParticleHandle->SetX(WorldTM.GetTranslation());
		ParticleHandle->SetR(WorldTM.GetRotation());

		FPBDRigidParticleHandle* Dynamic = ParticleHandle->CastToRigidParticle();
		if(Dynamic && Dynamic->ObjectState() == Chaos::EObjectStateType::Dynamic)
		{
			Dynamic->SetP(Dynamic->GetX());
			Dynamic->SetQf(Dynamic->GetRf());
			Dynamic->AuxilaryValue(ParticlePrevXs) = Dynamic->GetP();
			Dynamic->AuxilaryValue(ParticlePrevRs) = Dynamic->GetQ();
		}
	}

	bool FActorHandle::SetIsKinematic(bool bKinematic)
	{
		using namespace Chaos;

		if (ParticleHandle == nullptr)
		{
			return false;
		}

		EParticleType CurrentParticleType = ParticleHandle->GetParticleType();

		if (CurrentParticleType == EParticleType::Kinematic && bKinematic)
		{
			return true;
		}
		if (CurrentParticleType == EParticleType::Rigid)
		{
			if (FPBDRigidParticleHandle* Dynamic = ParticleHandle->CastToRigidParticle())
			{
				// Note that the state might be dynamic, sleeping, or kinematic
				if (Dynamic->ObjectState() != EObjectStateType::Kinematic && bKinematic)
				{
					Dynamic->SetObjectStateLowLevel(EObjectStateType::Kinematic);
					return true;
				}
				else if (Dynamic->ObjectState() == EObjectStateType::Kinematic && !bKinematic)
				{
					Dynamic->SetObjectStateLowLevel(EObjectStateType::Dynamic);
					return true;
				}
				return true;
			}
		}
		return false;
	}

	bool FActorHandle::GetIsKinematic() const
	{
		return Handle()->IsKinematic();
	}

	const FKinematicTarget& FActorHandle::GetKinematicTarget() const
	{
		check(ParticleHandle->CastToKinematicParticle());
		return ParticleHandle->CastToKinematicParticle()->KinematicTarget();
	}

	FKinematicTarget& FActorHandle::GetKinematicTarget()
	{
		check(ParticleHandle->CastToKinematicParticle());
		return ParticleHandle->CastToKinematicParticle()->KinematicTarget();
	}

	void FActorHandle::SetKinematicTarget(const FTransform& WorldTM)
	{
		using namespace Chaos;

		if (ensure(GetIsKinematic()))
		{
			FGenericParticleHandle GenericHandle(ParticleHandle);
			FTransform ParticleTransform = FParticleUtilities::ActorWorldToParticleWorld(GenericHandle, WorldTM);

			GetKinematicTarget().SetTargetMode(ParticleTransform);
		}

	}

	bool FActorHandle::HasKinematicTarget() const
	{
		if (GetIsKinematic())
		{
			return GetKinematicTarget().GetMode() == Chaos::EKinematicTargetMode::Position;
		}
		return false;
	}

	bool FActorHandle::IsSimulated() const
	{
		return ParticleHandle->CastToRigidParticle() != nullptr && ParticleHandle->ObjectState() == Chaos::EObjectStateType::Dynamic;
	}

	bool FActorHandle::CouldBeDynamic() const
	{
		return ParticleHandle->CastToRigidParticle() != nullptr;
	}

	void FActorHandle::SetUpdateKinematicFromSimulation(bool bUpdateKinematicFromSimulation)
	{
		using namespace Chaos;
		if (FPBDRigidParticleHandle* Rigid = Handle()->CastToRigidParticle())
		{
			Rigid->SetUpdateKinematicFromSimulation(bUpdateKinematicFromSimulation);
		}
	}

	bool FActorHandle::GetUpdateKinematicFromSimulation() const
	{
		using namespace Chaos;
		if (const FPBDRigidParticleHandle* Rigid = Handle()->CastToRigidParticle())
		{
			return Rigid->UpdateKinematicFromSimulation();
		}
		return false;
	}

	bool FActorHandle::IsGravityEnabled() const
	{
		using namespace Chaos;
		if (FPBDRigidParticleHandle* Rigid = Handle()->CastToRigidParticle())
		{
			return IsSimulated() && Rigid->GravityEnabled();
		}
		return false;
	}

	void FActorHandle::SetGravityEnabled(bool bEnable)
	{
		using namespace Chaos;
		if (FPBDRigidParticleHandle* Rigid = Handle()->CastToRigidParticle())
		{
			Rigid->SetGravityEnabled(bEnable);
		}
	}

	FTransform FActorHandle::GetWorldTransform() const
	{
		using namespace Chaos;

		return FParticleUtilities::GetActorWorldTransform(FGenericParticleHandle(ParticleHandle));
	}

	void FActorHandle::SetLinearVelocity(const FVector& NewLinearVelocity)
	{
		using namespace Chaos;

		if (FKinematicGeometryParticleHandle* KinematicParticleHandle = ParticleHandle->CastToKinematicParticle())
		{
			KinematicParticleHandle->SetV(NewLinearVelocity);
		}
	}

	FVector FActorHandle::GetLinearVelocity() const
	{
		return Handle()->V();
	}

	void FActorHandle::SetAngularVelocity(const FVector& NewAngularVelocity)
	{
		using namespace Chaos;

		if (FKinematicGeometryParticleHandle* KinematicParticleHandle = ParticleHandle->CastToKinematicParticle())
		{
			KinematicParticleHandle->SetW(NewAngularVelocity);
		}
	}

	FVector FActorHandle::GetAngularVelocity() const
	{
		return Handle()->W();
	}

	void FActorHandle::AddForce(const FVector& Force)
	{
		using namespace Chaos;

		if (FPBDRigidParticleHandle* Rigid = Handle()->CastToRigidParticle())
		{
			Rigid->AddForce(Force);
		}
	}

	void FActorHandle::AddTorque(const FVector& Torque)
	{
		using namespace Chaos;

		if (FPBDRigidParticleHandle* Rigid = Handle()->CastToRigidParticle())
		{
			Rigid->AddTorque(Torque);
		}
	}

	void FActorHandle::AddForce(const FVector& Linear, EForceType ForceType)
	{
		using namespace Chaos;

		if (FPBDRigidParticleHandle* Rigid = Handle()->CastToRigidParticle())
		{
			switch (ForceType)
			{
			case EForceType::AddForce:
				Rigid->AddForce(Linear);
				break;
			case EForceType::AddAcceleration:
				Rigid->Acceleration() += Linear;
				break;
			case EForceType::AddImpulse:
				Rigid->LinearImpulseVelocity() += Linear * Rigid->InvM();
				break;
			case EForceType::AddVelocity:
				Rigid->LinearImpulseVelocity() += Linear;
				break;
			default:
				break;
			}
		}
	}

	// Calculates the force which, when applied at R (relative to the CoM), results in an acceleration Accel
	// at that location.
	static FVector CalculateForceForAccelAtPoint(
		Chaos::FPBDRigidParticleHandle* Rigid, const FVector& R, const FVector& Accel)
	{
		const double A = Accel.Length();
		if (A > UE_SMALL_NUMBER)
		{
			const FVector Dir = Accel / A;
			// This is the resulting component of linear acceleration in the direction dir that
			// would be achieved by unit force
			const double Response = FVector::DotProduct(
				Dir * Rigid->InvM() + FVector::CrossProduct(Chaos::FParticleUtilities::GetWorldInvInertia(Rigid) * 
					FVector::CrossProduct(R, Dir), R), 
				Dir);
			if (Response > UE_SMALL_NUMBER)
			{
				return Dir * (A / Response);
			}
		}
		return FVector::ZeroVector;
	}

	void FActorHandle::AddForceAtLocation(const FVector& Linear, const FVector& Location, EForceType ForceType)
	{
		using namespace Chaos;

		if (FPBDRigidParticleHandle* Rigid = Handle()->CastToRigidParticle())
		{
			const FVector CoM = FParticleUtilities::GetCoMWorldPosition(Rigid);
			const FVector LocationRelCoM = Location - CoM;
			switch (ForceType)
			{
			case EForceType::AddForce:
			{
				Rigid->AddForce(Linear);
				Rigid->AddTorque(FVector::CrossProduct(LocationRelCoM, Linear));
			}
			break;
			case EForceType::AddAcceleration:
			{
				const FVector DesiredForce = CalculateForceForAccelAtPoint(Rigid, LocationRelCoM, Linear);
				Rigid->AddForce(DesiredForce);
				Rigid->AddTorque(FVector::CrossProduct(LocationRelCoM, DesiredForce));
			}
			break;
			case EForceType::AddImpulse:
			{
				Rigid->LinearImpulseVelocity() += Linear * Rigid->InvM();
				Rigid->AngularImpulseVelocity() += FParticleUtilities::GetWorldInvInertia(Rigid) *
					FVector::CrossProduct(LocationRelCoM, Linear);
			}
			break;
			case EForceType::AddVelocity:
			{
				// The accel->force calculation is the same as velChange->impulse
				const FVector DesiredImpulse = CalculateForceForAccelAtPoint(Rigid, LocationRelCoM, Linear);
				Rigid->LinearImpulseVelocity() += DesiredImpulse * Rigid->InvM();
				Rigid->AngularImpulseVelocity() += FParticleUtilities::GetWorldInvInertia(Rigid) *
					FVector::CrossProduct(LocationRelCoM, DesiredImpulse);
			}
			break;
			default:
				break;
			}
		}
	}

	void FActorHandle::AddTorque(const FVector& Angular, EForceType ForceType)
	{
		using namespace Chaos;

		if (FPBDRigidParticleHandle* Rigid = Handle()->CastToRigidParticle())
		{
			switch (ForceType)
			{
			case EForceType::AddForce:
				Rigid->AddTorque(Angular);
				break;
			case EForceType::AddAcceleration:
				Rigid->AngularAcceleration() += Angular;
				break;
			case EForceType::AddImpulse:
				Rigid->AngularImpulseVelocity() += FParticleUtilities::GetWorldInvInertia(Rigid) * Angular;
				break;
			case EForceType::AddVelocity:
				Rigid->AngularImpulseVelocity() += Angular;
				break;
			default:
				break;
			}
		}
	}

	void FActorHandle::AddRadialForce(const FVector& Origin, FReal Strength, FReal Radius, ERadialImpulseFalloff Falloff, EForceType ForceType)
	{
		using namespace Chaos;

		if (FPBDRigidParticleHandle* Rigid = Handle()->CastToRigidParticle())
		{
			const FRigidTransform3& PCOMTransform = FParticleUtilities::GetCoMWorldTransform(Rigid);
			FVec3 Delta = PCOMTransform.GetTranslation() - Origin;

			const FReal Mag = Delta.Size();
			if (Mag > Radius)
			{
				return;
			}
			Delta.Normalize();

			FReal ImpulseMag = Strength;
			if (Falloff == RIF_Linear)
			{
				ImpulseMag *= ((FReal)1. - (Mag / Radius));
			}

			const FVec3 PImpulse = Delta * ImpulseMag;
			const FVec3 ApplyDelta = (ForceType == EForceType::AddAcceleration || ForceType == EForceType::AddVelocity) ? PImpulse : PImpulse * Rigid->InvM();

			if (ForceType == EForceType::AddImpulse || ForceType == EForceType::AddVelocity)
			{
				Rigid->SetV(Rigid->GetV() + ApplyDelta);
			}
			else
			{
				Rigid->Acceleration() += ApplyDelta * Rigid->InvM();
			}
		}
	}

	void FActorHandle::AddImpulseAtLocation(FVector Impulse, FVector Location)
	{
		using namespace Chaos;

		if (FPBDRigidParticleHandle* Rigid = Handle()->CastToRigidParticle())
		{
			FVector CoM = FParticleUtilities::GetCoMWorldPosition(Rigid);
			Chaos::FMatrix33 InvInertia = FParticleUtilities::GetWorldInvInertia(Rigid);
			Rigid->LinearImpulseVelocity() += Impulse * Rigid->InvM();
			Rigid->AngularImpulseVelocity() += InvInertia * FVector::CrossProduct(Location - CoM, Impulse);
		}
	}

	void FActorHandle::SetLinearDamping(FReal NewLinearDamping)
	{
		using namespace Chaos;

		if (FPBDRigidParticleHandle* Rigid = Handle()->CastToRigidParticle())
		{
			Rigid->LinearEtherDrag() = NewLinearDamping;
		}
	}

	FReal FActorHandle::GetLinearDamping() const
	{
		using namespace Chaos;

		if (FPBDRigidParticleHandle* Rigid = Handle()->CastToRigidParticle())
		{
			return Rigid->LinearEtherDrag();
		}
		return 0.0f;
	}

	void FActorHandle::SetAngularDamping(FReal NewAngularDamping)
	{
		using namespace Chaos;

		if (FPBDRigidParticleHandle* Rigid = Handle()->CastToRigidParticle())
		{
			Rigid->AngularEtherDrag() = NewAngularDamping;
		}
	}

	FReal FActorHandle::GetAngularDamping() const
	{
		using namespace Chaos;

		if (FPBDRigidParticleHandle* Rigid = Handle()->CastToRigidParticle())
		{
			return Rigid->AngularEtherDrag();
		}
		return 0.0f;
	}

	void FActorHandle::SetMaxLinearVelocitySquared(FReal NewMaxLinearVelocitySquared)
	{
#if IMMEDIATEPHYSICS_CHAOS_TODO
#endif
	}

	FReal FActorHandle::GetMaxLinearVelocitySquared() const
	{
#if IMMEDIATEPHYSICS_CHAOS_TODO
#endif
		return FLT_MAX;
	}

	void FActorHandle::SetMaxAngularVelocitySquared(FReal NewMaxAngularVelocitySquared)
	{
#if IMMEDIATEPHYSICS_CHAOS_TODO
#endif
	}

	FReal FActorHandle::GetMaxAngularVelocitySquared() const
	{
#if IMMEDIATEPHYSICS_CHAOS_TODO
#endif
		return FLT_MAX;
	}

	void FActorHandle::SetInverseMass(FReal NewInverseMass)
	{
		using namespace Chaos;

		FPBDRigidParticleHandle* Dynamic = ParticleHandle->CastToRigidParticle();
		if(Dynamic && Dynamic->ObjectState() == EObjectStateType::Dynamic)
		{
			FReal NewMass = (NewInverseMass > UE_SMALL_NUMBER) ? (FReal)1. / NewInverseMass : (FReal)0.;
			Dynamic->SetM(NewMass);
			Dynamic->SetInvM(NewInverseMass);
		}
	}

	FReal FActorHandle::GetInverseMass() const
	{
		return Handle()->InvM();
	}

	FReal FActorHandle::GetMass() const
	{
		return Handle()->M();
	}

	void FActorHandle::SetInverseInertia(const FVector& NewInverseInertia)
	{
		using namespace Chaos;

		FPBDRigidParticleHandle* Dynamic = ParticleHandle->CastToRigidParticle();
		if(Dynamic && Dynamic->ObjectState() == EObjectStateType::Dynamic)
		{
			Chaos::FVec3 NewInertia = FVector3f::ZeroVector;
			if ((NewInverseInertia.X > UE_SMALL_NUMBER) && (NewInverseInertia.Y > UE_SMALL_NUMBER) && (NewInverseInertia.Z > UE_SMALL_NUMBER))
			{
				NewInertia = FVector3f( 1.0f / NewInverseInertia.X , 1.0f / NewInverseInertia.Y, 1.0f / NewInverseInertia.Z );
			}
			Dynamic->SetI(TVec3<FRealSingle>(NewInertia.X, NewInertia.Y, NewInertia.Z ));
			Dynamic->SetInvI(TVec3<FRealSingle>(NewInverseInertia.X, NewInverseInertia.Y, NewInverseInertia.Z ));
			
			if (Dynamic->InertiaConditioningEnabled())
			{
				Dynamic->SetInertiaConditioningDirty();
			}
		}
	}

	FVector FActorHandle::GetInverseInertia() const
	{
		return FVector(Handle()->InvI());
	}

	FVector FActorHandle::GetInertia() const
	{
		return FVector(Handle()->I());
	}

	void FActorHandle::SetMaxDepenetrationVelocity(FReal NewMaxDepenetrationVelocity)
	{
#if IMMEDIATEPHYSICS_CHAOS_TODO
#endif
	}

	FReal FActorHandle::GetMaxDepenetrationVelocity() const
	{
#if IMMEDIATEPHYSICS_CHAOS_TODO
#endif
		return FLT_MAX;
	}

	void FActorHandle::SetMaxContactImpulse(FReal NewMaxContactImpulse)
	{
#if IMMEDIATEPHYSICS_CHAOS_TODO
#endif
	}

	FReal FActorHandle::GetMaxContactImpulse() const
	{
#if IMMEDIATEPHYSICS_CHAOS_TODO
#endif
		return FLT_MAX;
	}

	FTransform FActorHandle::GetLocalCoMTransform() const
	{
		return FTransform(Handle()->RotationOfMass(), Handle()->CenterOfMass());
	}

	FVector FActorHandle::GetLocalCoMLocation() const
	{
		return Handle()->CenterOfMass();
	}

	void FActorHandle::SetCCDEnabled(bool bEnable)
	{
		using namespace Chaos;
		if (FPBDRigidParticleHandle* Dynamic = ParticleHandle->CastToRigidParticle())
		{
			Dynamic->SetCCDEnabled(bEnable);
		}
	}

	bool FActorHandle::GetCCDEnabled() const
	{
		using namespace Chaos;
		if (FPBDRigidParticleHandle* Dynamic = ParticleHandle->CastToRigidParticle())
		{
			return Dynamic->CCDEnabled();
		}
		return false;
	}

	int32 FActorHandle::GetLevel() const
	{
		return Level;
	}

	void FActorHandle::SetLevel(int32 InLevel)
	{
		Level = InLevel;
	}
}
