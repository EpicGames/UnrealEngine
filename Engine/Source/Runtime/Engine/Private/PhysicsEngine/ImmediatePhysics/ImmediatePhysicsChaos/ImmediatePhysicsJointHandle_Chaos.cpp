// Copyright Epic Games, Inc. All Rights Reserved.

#include "Physics/ImmediatePhysics/ImmediatePhysicsChaos/ImmediatePhysicsJointHandle_Chaos.h"
#include "Physics/ImmediatePhysics/ImmediatePhysicsChaos/ImmediatePhysicsActorHandle_Chaos.h"
#include "Physics/ImmediatePhysics/ImmediatePhysicsAdapters.h"

#include "Chaos/Particle/ParticleUtilities.h"
#include "Chaos/PBDJointConstraints.h"
#include "Chaos/ChaosConstraintSettings.h"

#include "PhysicsEngine/ConstraintInstance.h"

namespace ImmediatePhysics_Chaos
{
	// TODO: Deprecated - remove this and decl in ImmediatePhysicsJointHandle_Chaos.h
	void UpdateJointSettingsFromConstraintProfile(const FConstraintProfileProperties& Profile, Chaos::FPBDJointSettings& JointSettings)
	{
		return ImmediatePhysics::UpdateJointSettingsFromConstraintProfile(Profile, JointSettings);
	}


	FJointHandle::FJointHandle(FChaosConstraintContainer* InConstraints, FConstraintInstance* ConstraintInstance, FActorHandle* Actor1, FActorHandle* Actor2)
		: ActorHandles({ nullptr, nullptr })
		, Constraints(nullptr)
	{
		FJointSetup JointSetup = ImmediatePhysics::MakeJointSetup(ConstraintInstance, Actor1, Actor2);
		CreateJoint(InConstraints, JointSetup.ConstraintSettings, JointSetup.Actor1, JointSetup.Actor2);
	}

	FJointHandle::FJointHandle(FChaosConstraintContainer* InConstraints, const Chaos::FPBDJointSettings& ConstraintSettings, FActorHandle* Actor1, FActorHandle* Actor2)
		: ActorHandles({ nullptr, nullptr })
		, Constraints(nullptr)
	{
		CreateJoint(InConstraints, ConstraintSettings, Actor1, Actor2);
	}

	FJointHandle::FJointHandle(FChaosConstraintContainer* InConstraints, const FJointSetup& JointSetup)
		: ActorHandles({ nullptr, nullptr })
		, Constraints(nullptr)
	{
		CreateJoint(InConstraints, JointSetup.ConstraintSettings, JointSetup.Actor1, JointSetup.Actor2);
	}

	void FJointHandle::CreateJoint(FChaosConstraintContainer* InConstraints, const Chaos::FPBDJointSettings& ConstraintSettings, FActorHandle* Actor1, FActorHandle* Actor2)
	{
		Constraints = InConstraints;
		ActorHandles[0] = Actor1;
		ActorHandles[1] = Actor2;

		ConstraintHandle = Constraints->AddConstraint({ Actor1->ParticleHandle, Actor2->ParticleHandle }, ConstraintSettings);

		SetActorInertiaConditioningDirty();
	}

	FJointHandle::~FJointHandle()
	{
		ConstraintHandle->SetConstraintEnabled(false);
		ConstraintHandle->RemoveConstraint();
	}

	typename FJointHandle::FChaosConstraintHandle* FJointHandle::GetConstraint()
	{
		return ConstraintHandle;
	}
	
	const typename FJointHandle::FChaosConstraintHandle* FJointHandle::GetConstraint() const
	{
		return ConstraintHandle;
	}

	const Chaos::TVec2<FActorHandle*>& FJointHandle::GetActorHandles()
	{
		return ActorHandles;
	}

	const Chaos::TVec2<const FActorHandle*>& FJointHandle::GetActorHandles() const
	{
		return reinterpret_cast<const Chaos::TVec2<const FActorHandle*>&>(ActorHandles);
	}

	void FJointHandle::SetSoftLinearSettings(bool bLinearSoft, FReal LinearStiffness, FReal LinearDamping)
	{
		using namespace Chaos;
		Chaos::FPBDJointSettings JointSettings = ConstraintHandle->GetSettings();
		JointSettings.bSoftLinearLimitsEnabled = bLinearSoft;
		JointSettings.SoftLinearStiffness = bLinearSoft ? LinearStiffness : 0.0f;
		JointSettings.SoftLinearDamping = bLinearSoft ? LinearDamping : 0.0f;
		ConstraintHandle->SetSettings(JointSettings);
	}

	void FJointHandle::SetActorInertiaConditioningDirty()
	{
		using namespace Chaos;

		if (ActorHandles[0]->ParticleHandle != nullptr)
		{
			FGenericParticleHandle(ActorHandles[0]->ParticleHandle)->SetInertiaConditioningDirty();
		}

		if (ActorHandles[1]->ParticleHandle != nullptr)
		{
			FGenericParticleHandle(ActorHandles[1]->ParticleHandle)->SetInertiaConditioningDirty();
		}
	}
}

