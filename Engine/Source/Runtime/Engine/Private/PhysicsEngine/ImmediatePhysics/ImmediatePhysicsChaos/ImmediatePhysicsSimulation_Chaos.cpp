// Copyright Epic Games, Inc. All Rights Reserved.

#include "Physics/ImmediatePhysics/ImmediatePhysicsChaos/ImmediatePhysicsSimulation_Chaos.h"
#include "Chaos/PBDRigidsSOAs.h"
#include "Physics/ImmediatePhysics/ImmediatePhysicsAdapters.h"
#include "Physics/ImmediatePhysics/ImmediatePhysicsChaos/ImmediatePhysicsActorHandle_Chaos.h"
#include "Physics/ImmediatePhysics/ImmediatePhysicsChaos/ImmediatePhysicsJointHandle_Chaos.h"
#include "Physics/ImmediatePhysics/ImmediatePhysicsShared/ImmediatePhysicsCore.h"
#include "Physics/ImmediatePhysics/ImmediatePhysicsStats.h"

#include "PhysicalMaterials/PhysicalMaterial.h"
#include "PhysicsEngine/BodyInstance.h"
#include "PhysicsEngine/ConstraintInstance.h"

#include "Chaos/Collision/BasicCollisionDetector.h"
#include "Chaos/DebugDrawQueue.h"
#include "Chaos/Evolution/PBDMinEvolution.h"
#include "Chaos/Joint/ChaosJointLog.h"
#include "Chaos/MassConditioning.h"
#include "Chaos/PBDJointConstraints.h"
#include "ChaosDebugDraw/ChaosDDScene.h"
#include "ChaosDebugDraw/ChaosDDTimeline.h"
#include "ChaosVisualDebugger/ChaosVisualDebuggerTrace.h"
#include "Chaos/PBDJointConstraintUtilities.h"
#include "Stats/StatsTrace.h"

//UE_DISABLE_OPTIMIZATION

DECLARE_CYCLE_STAT(TEXT("FSimulation::Simulate_Chaos"), STAT_ImmediateSimulate_Chaos, STATGROUP_ImmediatePhysics);
DECLARE_CYCLE_STAT(TEXT("FSimulation::Simulate_Chaos::InertiaConditioning"), STAT_ImmediateSimulate_Chaos_InertiaConditioning, STATGROUP_ImmediatePhysics);

// Stat Counters
DECLARE_DWORD_COUNTER_STAT(TEXT("FSimulation::NumBodies"), STAT_ImmediateSimulate_ChaosCounter_NumBodies, STATGROUP_ImmediatePhysicsCounters);
DECLARE_DWORD_COUNTER_STAT(TEXT("FSimulation::NumDynamicBodies"), STAT_ImmediateSimulate_ChaosCounter_NumDynamicBodies, STATGROUP_ImmediatePhysicsCounters);
DECLARE_DWORD_COUNTER_STAT(TEXT("FSimulation::NumKinematicBodies"), STAT_ImmediateSimulate_ChaosCounter_NumKinematicBodies, STATGROUP_ImmediatePhysicsCounters);
DECLARE_DWORD_COUNTER_STAT(TEXT("FSimulation::NumContacts"), STAT_ImmediateSimulate_ChaosCounter_NumContacts, STATGROUP_ImmediatePhysicsCounters);
DECLARE_DWORD_COUNTER_STAT(TEXT("FSimulation::NumJoints"), STAT_ImmediateSimulate_ChaosCounter_NumJoints, STATGROUP_ImmediatePhysicsCounters);

//////////////////////////////////////////////////////////////////////////
// @todo(ccaulfield): remove when finished
//
Chaos::FRealSingle ChaosImmediate_Evolution_StepTime = 0.0f;
int32 ChaosImmediate_Evolution_NumSteps = 0;
Chaos::FRealSingle ChaosImmediate_Evolution_InitialStepTime = 0.033f;
int32 ChaosImmediate_Evolution_DeltaTimeCount = 100;
int32 ChaosImmediate_Evolution_PositionIterations = -1;
int32 ChaosImmediate_Evolution_VelocityIterations = -1;
int32 ChaosImmediate_Evolution_ProjectionIterations = -1;
int32 ChaosImmediate_DisableInactiveByIndex = 1;
int32 ChaosImmediate_Evolution_NumCollisionsPerBlock = 50;
FAutoConsoleVariableRef CVarChaosImmPhysStepTime(TEXT("p.Chaos.ImmPhys.StepTime"), ChaosImmediate_Evolution_StepTime, TEXT("Override step time (if not zero)"));
FAutoConsoleVariableRef CVarChaosImmPhysNumSteps(TEXT("p.Chaos.ImmPhys.NumSteps"), ChaosImmediate_Evolution_NumSteps, TEXT("Override num steps (if not zero)"));
FAutoConsoleVariableRef CVarChaosImmPhysInitialStepTime(TEXT("p.Chaos.ImmPhys.InitialStepTime"), ChaosImmediate_Evolution_InitialStepTime, TEXT("Initial step time (then calculated from rolling average)"));
FAutoConsoleVariableRef CVarChaosImmPhysDeltaTimeCount(TEXT("p.Chaos.ImmPhys.DeltaTimeCount"), ChaosImmediate_Evolution_DeltaTimeCount, TEXT("The number of ticks over which the moving average is calculated"));
FAutoConsoleVariableRef CVarChaosImmPhysPositionIterations(TEXT("p.Chaos.ImmPhys.PositionIterations"), ChaosImmediate_Evolution_PositionIterations, TEXT("Override number of position iteration loops in immediate physics (if >= 0)"));
FAutoConsoleVariableRef CVarChaosImmPhysVelocityIterations(TEXT("p.Chaos.ImmPhys.VelocityIterations"), ChaosImmediate_Evolution_VelocityIterations, TEXT("Override number of velocity iteration loops in immediate physics (if >= 0)"));
FAutoConsoleVariableRef CVarChaosImmPhysProjectionIterations(TEXT("p.Chaos.ImmPhys.ProjectionIterations"), ChaosImmediate_Evolution_ProjectionIterations, TEXT("Override number of projection iteration loops in immediate physics (if >= 0)"));
FAutoConsoleVariableRef CVarChaosImmPhysDisableInactiveByIndex(TEXT("p.Chaos.ImmPhys.DisableInactiveByIndex"), ChaosImmediate_DisableInactiveByIndex, TEXT("Disable bodies that are no longer active based on the index, rather than just count."));
FAutoConsoleVariableRef CVarChaosImmPhysNumCollisionsPerBlock(TEXT("p.Chaos.ImmPhys.NumCollisionsPerBlock"), ChaosImmediate_Evolution_NumCollisionsPerBlock, TEXT("The number of collision in a block in the collision pool. Higher values give better cache efficieny but waste memory if you do not need that many"));

Chaos::FRealSingle ChaosImmediate_Evolution_SimSpaceCentrifugalAlpha = 1.0f;
Chaos::FRealSingle ChaosImmediate_Evolution_SimSpaceCoriolisAlpha = 0.5f;
Chaos::FRealSingle ChaosImmediate_Evolution_SimSpaceEulerAlpha = 1.0f;
FAutoConsoleVariableRef CVarChaosImmPhysSimSpaceCentrifugalAlpha(TEXT("p.Chaos.ImmPhys.SimSpaceCentrifugalAlpha"), ChaosImmediate_Evolution_SimSpaceCentrifugalAlpha, TEXT("Settings for simulation space system for rigid body nodes"));
FAutoConsoleVariableRef CVarChaosImmPhysSimSpaceCoriolisAlpha(TEXT("p.Chaos.ImmPhys.SimSpaceCoriolisAlpha"), ChaosImmediate_Evolution_SimSpaceCoriolisAlpha, TEXT("Settings for simulation space system for rigid body nodes"));
FAutoConsoleVariableRef CVarChaosImmPhysSimSpaceEulerAlpha(TEXT("p.Chaos.ImmPhys.SimSpaceEulerAlpha"), ChaosImmediate_Evolution_SimSpaceEulerAlpha, TEXT("Settings for simulation space system for rigid body nodes"));

Chaos::FRealSingle ChaosImmediate_Evolution_MinStepTime = 0.01f;
Chaos::FRealSingle ChaosImmediate_Evolution_FixedStepTime = -1.0f;
Chaos::FRealSingle ChaosImmediate_Evolution_FixedStepTolerance = 0.05f;
FAutoConsoleVariableRef CVarChaosImmPhysMinStepTime(TEXT("p.Chaos.ImmPhys.MinStepTime"), ChaosImmediate_Evolution_MinStepTime, TEXT("If non-zero, then if step time is lower than this, go into fixed step mode with this timestep."));
FAutoConsoleVariableRef CVarChaosImmPhysFixedStepTime(TEXT("p.Chaos.ImmPhys.FixedStepTime"), ChaosImmediate_Evolution_FixedStepTime, TEXT("Override fixed step time mode: fixed step time (if positive); variable time mode (if zero); asset defined (if negative)"));
FAutoConsoleVariableRef CVarChaosImmPhysFixedStepTolerance(TEXT("p.Chaos.ImmPhys.FixedStepTolerance"), ChaosImmediate_Evolution_FixedStepTolerance, TEXT("Time remainder required to add a new step (fraction of FixedStepTime)"));

int32 ChaosImmediate_Collision_Enabled = 1;
int32 ChaosImmediate_Collision_NumPositionFrictionIterations = 0;		// No static friction for RBAN
int32 ChaosImmediate_Collision_NumVelocityFrictionIterations = 1;		// Dynamic friction for RBAN in velocity solve
int32 ChaosImmediate_Collision_NumPositionShockPropagationIterations = 0;
int32 ChaosImmediate_Collision_NumVelocityShockPropagationIterations = 0;
int32 ChaosImmediate_Collision_Priority = 1;
Chaos::FRealSingle ChaosImmediate_Collision_CullDistance = -1.0f;
Chaos::FRealSingle ChaosImmediate_Collision_MaxDepenetrationVelocity = -1.0f;
Chaos::FRealSingle ChaosImmediate_Collision_RestitutionThresholdMultiplier = 1.5f;
int32 ChaosImmediate_Collision_RestitutionEnabled = true;
int32 ChaosImmediate_Collision_DeferNarrowPhase = 1;
int32 ChaosImmediate_Collision_UseManifolds = -1;
FAutoConsoleVariableRef CVarChaosImmPhysCollisionDisable(TEXT("p.Chaos.ImmPhys.Collision.Enabled"), ChaosImmediate_Collision_Enabled, TEXT("Enable/Disable collisions in Immediate Physics."));
FAutoConsoleVariableRef CVarChaosImmPhysCollisionPriority(TEXT("p.Chaos.ImmPhys.Collision.Priority"), ChaosImmediate_Collision_Priority, TEXT("Set the Collision constraint sort order (Joints have priority 0)"));
FAutoConsoleVariableRef CVarChaosImmPhysCollisionCullDistance(TEXT("p.Chaos.ImmPhys.Collision.CullDistance"), ChaosImmediate_Collision_CullDistance, TEXT("Set the collision CullDistance (if >= 0)"));
FAutoConsoleVariableRef CVarChaosImmPhysCollisionMaxDepenetrationVelocity(TEXT("p.Chaos.ImmPhys.Collision.MaxDepenetrationVelocity"), ChaosImmediate_Collision_MaxDepenetrationVelocity, TEXT("Set the collision Max Depenetration Velocity (if >= 0)"));
FAutoConsoleVariableRef CVarChaosImmPhysCollisionRestitutionThresholdMultiplier(TEXT("p.Chaos.ImmPhys.Collision.RestitutionThresholdMultiplier"), ChaosImmediate_Collision_RestitutionThresholdMultiplier, TEXT("Collision Restitution Threshold (Acceleration) = Multiplier * Gravity"));
FAutoConsoleVariableRef CVarChaosImmPhysCollisionRestitutionEnabled(TEXT("p.Chaos.ImmPhys.Collision.RestitutionEnabled"), ChaosImmediate_Collision_RestitutionEnabled, TEXT("Collision Restitution Enable/Disable"));
FAutoConsoleVariableRef CVarChaosImmPhysCollisionDeferNarrowPhase(TEXT("p.Chaos.ImmPhys.Collision.DeferNarrowPhase"), ChaosImmediate_Collision_DeferNarrowPhase, TEXT("[Legacy Solver] Create contacts for all broadphase pairs, perform NarrowPhase later."));
FAutoConsoleVariableRef CVarChaosImmPhysCollisionUseManifolds(TEXT("p.Chaos.ImmPhys.Collision.UseManifolds"), ChaosImmediate_Collision_UseManifolds, TEXT("Override Enable/Disable use of manifoldes in collision (if >= 0)."));

Chaos::FRealSingle ChaosImmediate_Joint_SwingTwistAngleTolerance = 1.0e-6f;
Chaos::FRealSingle ChaosImmediate_Joint_PositionTolerance = 0.025f;
Chaos::FRealSingle ChaosImmediate_Joint_AngleTolerance = 0.001f;
int32 ChaosImmediate_Joint_NumShockPropagationIterations = -1;
int32 ChaosImmediate_Joint_SolvePositionLast = 1;
int32 ChaosImmediate_Joint_UsePositionBasedDrives = 1;
int32 ChaosImmediate_Joint_EnableTwistLimits = 1;
int32 ChaosImmediate_Joint_EnableSwingLimits = 1;
int32 ChaosImmediate_Joint_EnableDrives = 1;
Chaos::FRealSingle ChaosImmediate_Joint_LinearProjection = -1.0f;
Chaos::FRealSingle ChaosImmediate_Joint_AngularProjection = -1.0f;
Chaos::FRealSingle ChaosImmediate_Joint_ShockPropagation = -1.0f;
Chaos::FRealSingle ChaosImmediate_Joint_Stiffness = -1.0f;
Chaos::FRealSingle ChaosImmediate_Joint_SoftLinearStiffness = -1.0f;
Chaos::FRealSingle ChaosImmediate_Joint_SoftTwistStiffness = -1.0f;
Chaos::FRealSingle ChaosImmediate_Joint_SoftTwistDamping = -1.0f;
Chaos::FRealSingle ChaosImmediate_Joint_SoftSwingStiffness = -1.0f;
Chaos::FRealSingle ChaosImmediate_Joint_SoftSwingDamping = -1.0f;
Chaos::FRealSingle ChaosImmediate_Joint_LinearDriveStiffness = -1.0f;
Chaos::FRealSingle ChaosImmediate_Joint_LinearDriveDamping = -1.0f;
Chaos::FRealSingle ChaosImmediate_Joint_AngularDriveStiffness = -1.0f;
Chaos::FRealSingle ChaosImmediate_Joint_AngularDriveDamping = -1.0f;
Chaos::FRealSingle ChaosImmediate_Joint_MinParentMassRatio = 0.2f;
Chaos::FRealSingle ChaosImmediate_Joint_MaxInertiaRatio = 5.0f;
FAutoConsoleVariableRef CVarChaosImmPhysJointSwingTwistAngleTolerance(TEXT("p.Chaos.ImmPhys.Joint.SwingTwistAngleTolerance"), ChaosImmediate_Joint_SwingTwistAngleTolerance, TEXT("SwingTwistAngleTolerance."));
FAutoConsoleVariableRef CVarChaosImmPhysJointPositionTolerance(TEXT("p.Chaos.ImmPhys.Joint.PositionTolerance"), ChaosImmediate_Joint_PositionTolerance, TEXT("PositionTolerance."));
FAutoConsoleVariableRef CVarChaosImmPhysJointAngleTolerance(TEXT("p.Chaos.ImmPhys.Joint.AngleTolerance"), ChaosImmediate_Joint_AngleTolerance, TEXT("AngleTolerance."));
FAutoConsoleVariableRef CVarChaosImmPhysJointNumShockPropagationIterations(TEXT("p.Chaos.ImmPhys.Joint.NumShockPropagationIterations"), ChaosImmediate_Joint_NumShockPropagationIterations, TEXT("How many iterations to run shock propagation for"));
FAutoConsoleVariableRef CVarChaosImmPhysJointSolvePositionLast(TEXT("p.Chaos.ImmPhys.Joint.SolvePositionLast"), ChaosImmediate_Joint_SolvePositionLast, TEXT("Should we solve joints in position-then-rotation order (false) rotation-then-position order (true, default)"));
FAutoConsoleVariableRef CVarChaosImmPhysJointUsePBDVelocityDrives(TEXT("p.Chaos.ImmPhys.Joint.UsePBDDrives"), ChaosImmediate_Joint_UsePositionBasedDrives, TEXT("Whether to solve drives in the position or velocity phase of the solver (default true)"));
FAutoConsoleVariableRef CVarChaosImmPhysJointEnableTwistLimits(TEXT("p.Chaos.ImmPhys.Joint.EnableTwistLimits"), ChaosImmediate_Joint_EnableTwistLimits, TEXT("EnableTwistLimits."));
FAutoConsoleVariableRef CVarChaosImmPhysJointEnableSwingLimits(TEXT("p.Chaos.ImmPhys.Joint.EnableSwingLimits"), ChaosImmediate_Joint_EnableSwingLimits, TEXT("EnableSwingLimits."));
FAutoConsoleVariableRef CVarChaosImmPhysJointEnableDrives(TEXT("p.Chaos.ImmPhys.Joint.EnableDrives"), ChaosImmediate_Joint_EnableDrives, TEXT("EnableDrives."));
FAutoConsoleVariableRef CVarChaosImmPhysJointLinearProjection(TEXT("p.Chaos.ImmPhys.Joint.LinearProjection"), ChaosImmediate_Joint_LinearProjection, TEXT("6Dof joint projection amount override (if >= 0)."));
FAutoConsoleVariableRef CVarChaosImmPhysJointAngularProjection(TEXT("p.Chaos.ImmPhys.Joint.AngularProjection"), ChaosImmediate_Joint_AngularProjection, TEXT("6Dof joint projection amount override (if >= 0)."));
FAutoConsoleVariableRef CVarChaosImmPhysJointShockPropagation(TEXT("p.Chaos.ImmPhys.Joint.ShockPropagation"), ChaosImmediate_Joint_ShockPropagation, TEXT("6Dof joint shock propagation override (if >= 0)."));
FAutoConsoleVariableRef CVarChaosImmPhysJointStiffness(TEXT("p.Chaos.ImmPhys.Joint.Stiffness"), ChaosImmediate_Joint_Stiffness, TEXT("6Dof joint stiffness override (if > 0)."));
FAutoConsoleVariableRef CVarChaosImmPhysJointSoftLinearStiffness(TEXT("p.Chaos.ImmPhys.Joint.SoftLinearStiffness"), ChaosImmediate_Joint_SoftLinearStiffness, TEXT("6Dof joint soft linear stiffness override (if > 0)."));
FAutoConsoleVariableRef CVarChaosImmPhysJointSoftTwistStiffness(TEXT("p.Chaos.ImmPhys.Joint.SoftTwistStiffness"), ChaosImmediate_Joint_SoftTwistStiffness, TEXT("6Dof joint SoftTwist stiffness override (if > 0)."));
FAutoConsoleVariableRef CVarChaosImmPhysJointSoftTwistDamping(TEXT("p.Chaos.ImmPhys.Joint.SoftTwistDamping"), ChaosImmediate_Joint_SoftTwistDamping, TEXT("6Dof joint SoftTwist damping override (if > 0)."));
FAutoConsoleVariableRef CVarChaosImmPhysJointSoftSwingStiffness(TEXT("p.Chaos.ImmPhys.Joint.SoftSwingStiffness"), ChaosImmediate_Joint_SoftSwingStiffness, TEXT("6Dof joint SoftSwing stiffness override (if > 0)."));
FAutoConsoleVariableRef CVarChaosImmPhysJointSoftSwingDamping(TEXT("p.Chaos.ImmPhys.Joint.SoftSwingDamping"), ChaosImmediate_Joint_SoftSwingDamping, TEXT("6Dof joint SoftSwing damping override (if > 0)."));
FAutoConsoleVariableRef CVarChaosImmPhysJointLinearDriveStiffness(TEXT("p.Chaos.ImmPhys.Joint.LinearDriveStiffness"), ChaosImmediate_Joint_LinearDriveStiffness, TEXT("6Dof joint drive stiffness override (if > 0)."));
FAutoConsoleVariableRef CVarChaosImmPhysJointLinearDriveDamping(TEXT("p.Chaos.ImmPhys.Joint.LinearDriveDamping"), ChaosImmediate_Joint_LinearDriveDamping, TEXT("6Dof joint drive damping override (if > 0)."));
FAutoConsoleVariableRef CVarChaosImmPhysJointAngularDriveStiffness(TEXT("p.Chaos.ImmPhys.Joint.AngularDriveStiffness"), ChaosImmediate_Joint_AngularDriveStiffness, TEXT("6Dof joint drive stiffness override (if > 0)."));
FAutoConsoleVariableRef CVarChaosImmPhysJointAngularDriveDamping(TEXT("p.Chaos.ImmPhys.Joint.AngularDriveDamping"), ChaosImmediate_Joint_AngularDriveDamping, TEXT("6Dof joint drive damping override (if > 0)."));
FAutoConsoleVariableRef CVarChaosImmPhysJointMinParentMassRatio(TEXT("p.Chaos.ImmPhys.Joint.MinParentMassRatio"), ChaosImmediate_Joint_MinParentMassRatio, TEXT("6Dof joint MinParentMassRatio (if > 0)"));
FAutoConsoleVariableRef CVarChaosImmPhysJointMaxInertiaRatio(TEXT("p.Chaos.ImmPhys.Joint.MaxInertiaRatio"), ChaosImmediate_Joint_MaxInertiaRatio, TEXT("6Dof joint MaxInertiaRatio (if > 0)"));

bool bChaosImmediate_LinearInertiaConditioningEnabled = true;
FAutoConsoleVariableRef  CVarChaosImmPhysParticleLinearInertiaConditioningEnabled(TEXT("p.Chaos.ImmPhys.InertiaConditioning.LinearEnabled"), bChaosImmediate_LinearInertiaConditioningEnabled, TEXT("Enable/Disable constraint stabilization through inertia conditioning when using the linear joint solver"));

bool bChaosImmediate_NonLinearInertiaConditioningEnabled = false;
FAutoConsoleVariableRef  CVarChaosImmPhysParticleNonLinearInertiaConditioningEnabled(TEXT("p.Chaos.ImmPhys.InertiaConditioning.NonlinearEnabled"), bChaosImmediate_NonLinearInertiaConditioningEnabled, TEXT("Enable/Disable constraint stabilization through inertia conditioning when using the non-linear joint solver"));

Chaos::FRealSingle ChaosImmediate_InertiaConditioningDistance = 10;
FAutoConsoleVariableRef  CVarChaosImmPhysParticleInertiaConditioningDistance(TEXT("p.Chaos.ImmPhys.InertiaConditioning.Distance"), ChaosImmediate_InertiaConditioningDistance, TEXT("An input to inertia conditioning system. The joint distance error which needs to be stable (generate a low rotation)."));

Chaos::FRealSingle ChaosImmediate_InertiaConditioningRotationRatio = 2;
FAutoConsoleVariableRef  CVarChaosImmPhysParticleInertiaConditioningRotationRatio(TEXT("p.Chaos.ImmPhys.InertiaConditioning.RotationRatio"), ChaosImmediate_InertiaConditioningRotationRatio, TEXT("An input to inertia conditioning system. The maximum ratio of joint correction from rotation versus translation"));

Chaos::FRealSingle ChaosImmediate_MaxInvInertiaComponentRatio = 0;
FAutoConsoleVariableRef  CVarChaosImmPhysInertiaConditioningMaxInvInertiaComponentRatio(TEXT("p.Chaos.ImmPhys.InertiaConditioning.MaxInvInertiaComponentRatio"), ChaosImmediate_MaxInvInertiaComponentRatio, TEXT("An input to inertia conditioning system. The largest inertia component must be at least least multiple of the smallest component"));

// Whether to use the linear joint solver which is significantly faster than the non-linear one but less accurate. Only applies to the QuasiPBD Solver
int32 ChaosImmediate_Joint_UseLinearSolver = -1;
FAutoConsoleVariableRef CVarChaosImmPhysJointUseCachedSolver(TEXT("p.Chaos.ImmPhys.Joint.UseLinearSolver"), ChaosImmediate_Joint_UseLinearSolver, TEXT("Force use of linear or non-linear joint solver. (-1 to use PhysicsAsset setting)"));

bool bChaosImmediate_UseSimdForLinearSolver = true;
FAutoConsoleVariableRef  CVarChaosImmPhysUseSimdForLinearSolver(TEXT("p.Chaos.ImmPhys.UseSimd"), bChaosImmediate_UseSimdForLinearSolver, TEXT("Enable/Disable SIMD on the linear joint solver"));

//
// end remove when finished
//
//////////////////////////////////////////////////////////////////////////


// DebugDraw CVars
#if CHAOS_DEBUG_DRAW
bool bChaosImmediate_DebugDrawOnSimulate = false;
bool bChaosImmediate_DebugDrawParticles = false;
bool bChaosImmediate_DebugDrawShapes = false;
bool bChaosImmediate_DebugDrawShowStatics = true;
bool bChaosImmediate_DebugDrawShowKinematics = true;
bool bChaosImmediate_DebugDrawShowDynamics = true;
bool bChaosImmediate_DebugDrawBounds = false;
bool bChaosImmediate_DebugDrawCollisions = false;
bool bChaosImmediate_DebugDrawJoints = false;
bool bChaosImmediate_DebugDrawSimulationSpace = 0;
Chaos::DebugDraw::FChaosDebugDrawJointFeatures ChaosImmediate_DebugDrawJointFeatures = Chaos::DebugDraw::FChaosDebugDrawJointFeatures::MakeDefault();
FAutoConsoleVariableRef CVarChaosImmPhysDebugDrawEnabled(TEXT("p.Chaos.ImmPhys.DebugDrawOnSimulate"), bChaosImmediate_DebugDrawOnSimulate, TEXT("Enables debug drawing after the simulation completes."), ECVF_Default);
FAutoConsoleVariableRef CVarChaosImmPhysDebugDrawParticles(TEXT("p.Chaos.ImmPhys.DebugDrawParticles"), bChaosImmediate_DebugDrawParticles, TEXT("Whether to draw particles when debug drawing."), ECVF_Default);
FAutoConsoleVariableRef CVarChaosImmPhysDebugDrawShapes(TEXT("p.Chaos.ImmPhys.DebugDrawShapes"), bChaosImmediate_DebugDrawShapes, TEXT("Whether to draw shapes when debug drawing."), ECVF_Default);
FAutoConsoleVariableRef CVarChaosImmPhysDebugDrawBounds(TEXT("p.Chaos.ImmPhys.DebugDrawBounds"), bChaosImmediate_DebugDrawBounds, TEXT("Whether to draw bounds when debug drawing."), ECVF_Default);
FAutoConsoleVariableRef CVarChaosImmPhysDebugDrawCollisions(TEXT("p.Chaos.ImmPhys.DebugDrawCollisions"), bChaosImmediate_DebugDrawCollisions, TEXT("Whether to draw collisions when debug drawing."), ECVF_Default);
FAutoConsoleVariableRef CVarChaosImmPhysDebugDrawJoints(TEXT("p.Chaos.ImmPhys.DebugDrawJoints"), bChaosImmediate_DebugDrawJoints, TEXT("Whether to draw joints when debug drawing."), ECVF_Default);
FAutoConsoleVariableRef CVarChaosImmPhysDebugDrawSimulationSpace(TEXT("p.Chaos.ImmPhys.DebugDrawSimulationSpace"), bChaosImmediate_DebugDrawSimulationSpace, TEXT("Whether to draw the simulation frame of reference, acceleration and velocity when debug drawing."), ECVF_Default);
FAutoConsoleVariableRef CVarChaosImmPhysDebugDrawShowStatics(TEXT("p.Chaos.ImmPhys.DebugDrawShowStatics"), bChaosImmediate_DebugDrawShowStatics, TEXT("Show statics if shape debug draw is enabled"), ECVF_Default);
FAutoConsoleVariableRef CVarChaosImmPhysDebugDrawShowKinematics(TEXT("p.Chaos.ImmPhys.DebugDrawShowKinematics"), bChaosImmediate_DebugDrawShowKinematics, TEXT("Show kinematics if shape debug draw is enabled"), ECVF_Default);
FAutoConsoleVariableRef CVarChaosImmPhysDebugDrawShowDynamics(TEXT("p.Chaos.ImmPhys.DebugDrawShowDynamics"), bChaosImmediate_DebugDrawShowDynamics, TEXT("Show dynamics if shape debug draw is enabled"), ECVF_Default);
FAutoConsoleVariableRef CVarChaosImmPhysDebugDrawJointFeaturesCoMConnector(TEXT("p.Chaos.ImmPhys.DebugDraw.JointFeatures.CoMConnector"), ChaosImmediate_DebugDrawJointFeatures.bCoMConnector, TEXT("Joint features mask (see FDebugDrawJointFeatures)."));
FAutoConsoleVariableRef CVarChaosImmPhysDebugDrawJointFeaturesActorConnector(TEXT("p.Chaos.ImmPhys.DebugDraw.JointFeatures.ActorConnector"), ChaosImmediate_DebugDrawJointFeatures.bActorConnector, TEXT("Joint features mask (see FDebugDrawJointFeatures)."));
FAutoConsoleVariableRef CVarChaosImmPhysDebugDrawJointFeaturesStretch(TEXT("p.Chaos.ImmPhys.DebugDraw.JointFeatures.Stretch"), ChaosImmediate_DebugDrawJointFeatures.bStretch, TEXT("Joint features mask (see FDebugDrawJointFeatures)."));
FAutoConsoleVariableRef CVarChaosImmPhysDebugDrawJointFeaturesAxes(TEXT("p.Chaos.ImmPhys.DebugDraw.JointFeatures.Axes"), ChaosImmediate_DebugDrawJointFeatures.bAxes, TEXT("Joint features mask (see FDebugDrawJointFeatures)."));
FAutoConsoleVariableRef CVarChaosImmPhysDebugDrawJointFeaturesLevel(TEXT("p.Chaos.ImmPhys.DebugDraw.JointFeatures.Level"), ChaosImmediate_DebugDrawJointFeatures.bLevel, TEXT("Joint features mask (see FDebugDrawJointFeatures)."));
FAutoConsoleVariableRef CVarChaosImmPhysDebugDrawJointFeaturesIndex(TEXT("p.Chaos.ImmPhys.DebugDraw.JointFeatures.Index"), ChaosImmediate_DebugDrawJointFeatures.bIndex, TEXT("Joint features mask (see FDebugDrawJointFeatures)."));
FAutoConsoleVariableRef CVarChaosImmPhysDebugDrawJointFeaturesColor(TEXT("p.Chaos.ImmPhys.DebugDraw.JointFeatures.Color"), ChaosImmediate_DebugDrawJointFeatures.bColor, TEXT("Joint features mask (see FDebugDrawJointFeatures)."));
FAutoConsoleVariableRef CVarChaosImmPhysDebugDrawJointFeaturesIsland(TEXT("p.Chaos.ImmPhys.DebugDraw.JointFeatures.Island"), ChaosImmediate_DebugDrawJointFeatures.bIsland, TEXT("Joint features mask (see FDebugDrawJointFeatures)."));

Chaos::DebugDraw::FChaosDebugDrawSettings ChaosImmPhysDebugDebugDrawSettings(
	/* ArrowSize =					*/ 1.5f,
	/* BodyAxisLen =				*/ 4.0f,
	/* ContactLen =					*/ 4.0f,
	/* ContactWidth =				*/ 2.0f,
	/* ContactInfoWidth				*/ 2.0f,
	/* ContactOwnerWidth =			*/ 0.0f,
	/* ConstraintAxisLen =			*/ 5.0f,
	/* JointComSize =				*/ 2.0f,
	/* LineThickness =				*/ 0.15f,
	/* DrawScale =					*/ 1.0f,
	/* FontHeight =					*/ 10.0f,
	/* FontScale =					*/ 1.5f,
	/* ShapeThicknesScale =			*/ 1.0f,
	/* PointSize =					*/ 2.0f,
	/* VelScale =					*/ 0.0f,
	/* AngVelScale =				*/ 0.0f,
	/* ImpulseScale =				*/ 0.0f,
	/* PushOutScale =				*/ 0.0f,
	/* InertiaScale =				*/ 0.0f,
	/* DrawPriority =				*/ 10.0f,
	/* bShowSimple =				*/ true,
	/* bShowComplex =				*/ false,
	/* bInShowLevelSetCollision =	*/ false,
	/* InShapesColorsPerState =     */ Chaos::DebugDraw::GetDefaultShapesColorsByState(),
	/* InShapesColorsPerShaepType=  */ Chaos::DebugDraw::GetDefaultShapesColorsByShapeType(),
	/* InBoundsColorsPerState =     */ Chaos::DebugDraw::GetDefaultBoundsColorsByState(),
	/* InBoundsColorsPerShapeType=  */ Chaos::DebugDraw::GetDefaultBoundsColorsByShapeType()
);

FAutoConsoleVariableRef CVarChaosImmPhysArrowSize(TEXT("p.Chaos.ImmPhys.DebugDraw.ArrowSize"), ChaosImmPhysDebugDebugDrawSettings.ArrowSize, TEXT("ArrowSize."));
FAutoConsoleVariableRef CVarChaosImmPhysBodyAxisLen(TEXT("p.Chaos.ImmPhys.DebugDraw.BodyAxisLen"), ChaosImmPhysDebugDebugDrawSettings.BodyAxisLen, TEXT("BodyAxisLen."));
FAutoConsoleVariableRef CVarChaosImmPhysContactLen(TEXT("p.Chaos.ImmPhys.DebugDraw.ContactLen"), ChaosImmPhysDebugDebugDrawSettings.ContactLen, TEXT("ContactLen."));
FAutoConsoleVariableRef CVarChaosImmPhysContactWidth(TEXT("p.Chaos.ImmPhys.DebugDraw.ContactWidth"), ChaosImmPhysDebugDebugDrawSettings.ContactWidth, TEXT("ContactWidth."));
FAutoConsoleVariableRef CVarChaosImmPhysContactInfoWidth(TEXT("p.Chaos.ImmPhys.DebugDraw.ContactInfoWidth"), ChaosImmPhysDebugDebugDrawSettings.ContactInfoWidth, TEXT("ContactInfoWidth."));
FAutoConsoleVariableRef CVarChaosImmPhysContactOwnerWidth(TEXT("p.Chaos.ImmPhys.DebugDraw.ContactOwnerWidth"), ChaosImmPhysDebugDebugDrawSettings.ContactOwnerWidth, TEXT("ContactOwnerWidth."));
FAutoConsoleVariableRef CVarChaosImmPhysConstraintAxisLen(TEXT("p.Chaos.ImmPhys.DebugDraw.ConstraintAxisLen"), ChaosImmPhysDebugDebugDrawSettings.ConstraintAxisLen, TEXT("ConstraintAxisLen."));
FAutoConsoleVariableRef CVarChaosImmPhysLineThickness(TEXT("p.Chaos.ImmPhys.DebugDraw.LineThickness"), ChaosImmPhysDebugDebugDrawSettings.LineThickness, TEXT("LineThickness."));
FAutoConsoleVariableRef CVarChaosImmPhysLineShapeThickness(TEXT("p.Chaos.ImmPhys.DebugDraw.ShapeLineThicknessScale"), ChaosImmPhysDebugDebugDrawSettings.ShapeThicknesScale, TEXT("Shape lineThickness multiplier."));
FAutoConsoleVariableRef CVarChaosImmPhysVelScale(TEXT("p.Chaos.ImmPhys.DebugDraw.VelScale"), ChaosImmPhysDebugDebugDrawSettings.VelScale, TEXT("If >0 show velocity when drawing particle transforms."));
FAutoConsoleVariableRef CVarChaosImmPhysAngVelScale(TEXT("p.Chaos.ImmPhys.DebugDraw.AngVelScale"), ChaosImmPhysDebugDebugDrawSettings.AngVelScale, TEXT("If >0 show angular velocity when drawing particle transforms."));
FAutoConsoleVariableRef CVarChaosImmPhysImpulseScale(TEXT("p.Chaos.ImmPhys.DebugDraw.ImpulseScale"), ChaosImmPhysDebugDebugDrawSettings.ImpulseScale, TEXT("If >0 show impulses when drawing collisions."));
FAutoConsoleVariableRef CVarChaosImmPhysPushOutScale(TEXT("p.Chaos.ImmPhys.DebugDraw.PushOutScale"), ChaosImmPhysDebugDebugDrawSettings.PushOutScale, TEXT("If >0 show pushouts when drawing collisions."));
FAutoConsoleVariableRef CVarChaosImmPhysInertiaScale(TEXT("p.Chaos.ImmPhys.DebugDraw.InertiaScale"), ChaosImmPhysDebugDebugDrawSettings.InertiaScale, TEXT("If >0 show inertia when drawing particles."));
FAutoConsoleVariableRef CVarChaosImmPhysScale(TEXT("p.Chaos.ImmPhys.DebugDraw.Scale"), ChaosImmPhysDebugDebugDrawSettings.DrawScale, TEXT("Scale applied to all Chaos Debug Draw line lengths etc."));
#endif

namespace ImmediatePhysics_Chaos
{
	class FSimpleParticleUniqueIndices : public Chaos::IParticleUniqueIndices
	{
	public:
		Chaos::FUniqueIdx GenerateUniqueIdx() override
		{
			if (FreeIndices.Num())
			{
				const int32 FreeIndex = FreeIndices.Pop();
				return Chaos::FUniqueIdx(FreeIndex);
			}

			Chaos::FUniqueIdx NewUniqueIndex(NextUniqueIndex);
			NextUniqueIndex++;
			return NewUniqueIndex;
		}

		void ReleaseIdx(Chaos::FUniqueIdx Unique) override
		{
			ensure(Unique.IsValid());
			FreeIndices.Add(Unique.Idx);
		}

		~FSimpleParticleUniqueIndices() = default;

	private:
		int32 NextUniqueIndex = 0; // this includes all valid and freed indices
		TArray<int32> FreeIndices;
	};


	struct FSimulation::FImplementation
	{
	public:
		FImplementation()
			: Particles(UniqueIndices)
			, Joints()
			, Collisions(Particles, CollidedParticles, ParticleMaterials, PerParticleMaterials, nullptr, ChaosImmediate_Evolution_NumCollisionsPerBlock, 2000)
			, BroadPhase(&ActivePotentiallyCollidingPairs, nullptr, nullptr)
			, CollisionDetector(BroadPhase, Collisions)
			, Evolution(Particles, ParticlePrevXs, ParticlePrevRs, CollisionDetector)
			, NumActiveDynamicActorHandles(0)
			, SimulationSpace()
			, SimulationSpaceTransform(FTransform::Identity)
			, RollingAverageStepTime(ChaosImmediate_Evolution_InitialStepTime)
			, NumRollingAverageStepTimes(1)
			, MaxNumRollingAverageStepTimes(ChaosImmediate_Evolution_DeltaTimeCount)
			, bActorsDirty(false)
		{
			Particles.GetParticleHandles().AddArray(&CollidedParticles);
			Particles.GetParticleHandles().AddArray(&ParticleMaterials);
			Particles.GetParticleHandles().AddArray(&PerParticleMaterials);
			Particles.GetParticleHandles().AddArray(&ParticlePrevXs);
			Particles.GetParticleHandles().AddArray(&ParticlePrevRs);

			Evolution.AddConstraintContainer(Collisions, ChaosImmediate_Collision_Priority);
			Evolution.AddConstraintContainer(Joints.LinearConstraints, 0);
			Evolution.AddConstraintContainer(Joints.NonLinearConstraints, 0);
		}

		~FImplementation()
		{
		}

		// @todo(ccaulfield): Look into these...
		TArray<Chaos::FParticlePair> ActivePotentiallyCollidingPairs;
		TArray<FActorHandle*> StaticParticles;
		Chaos::TArrayCollectionArray<bool> CollidedParticles;
		Chaos::TArrayCollectionArray<Chaos::TSerializablePtr<Chaos::FChaosPhysicsMaterial>> ParticleMaterials;
		Chaos::TArrayCollectionArray<TUniquePtr<Chaos::FChaosPhysicsMaterial>> PerParticleMaterials;
		Chaos::TArrayCollectionArray<Chaos::FVec3> ParticlePrevXs;
		Chaos::TArrayCollectionArray<Chaos::FRotation3> ParticlePrevRs;

		FSimpleParticleUniqueIndices UniqueIndices;
		Chaos::FPBDRigidsSOAs Particles;
		Chaos::FPBDJointCombinedConstraints Joints;
		Chaos::FPBDCollisionConstraints Collisions;
		Chaos::FBasicBroadPhase BroadPhase;
		Chaos::FBasicCollisionDetector CollisionDetector;
		Chaos::FPBDMinEvolution Evolution;

		/** Mapping from entity index to handle */
		// @todo(ccaulfield): we now have handles pointing to handles which is inefficient - we can do better than this, but don't want to change API yet
		TArray<FActorHandle*> ActorHandles;
		int32 NumActiveDynamicActorHandles;

		/** Mapping from constraint index to handle */
		TArray<FJointHandle*> JointHandles;

		/** Slow to access. */
		// @todo(ccaulfield): Optimize
		TMap<const Chaos::FGeometryParticleHandle*, TSet<const Chaos::FGeometryParticleHandle*>> IgnoreCollisionParticlePairTable;

		TArray<Chaos::FParticlePair> PotentiallyCollidingPairs;

		// The motion of the simulation space for calculation of phantom forces
		Chaos::FSimulationSpace SimulationSpace;

		// The simulation space transform, used for debug draw
		FTransform SimulationSpaceTransform;

		Chaos::FReal FixedStepTime;
		Chaos::FReal RollingAverageStepTime;
		int32 NumRollingAverageStepTimes;
		int32 MaxNumRollingAverageStepTimes; // Set to 1 to disable smoothing
		bool bUseMinStepTime = true;
		bool bUseFixedStepTolerance = true;

		bool bActorsDirty;
	};

	//
	//
	//

	FSimulation::FSimulation()
	{
		using namespace Chaos;

		Implementation = MakeUnique<FImplementation>();

#if CHAOS_SOLVER_DEBUG_NAME
		DebugName = TEXT("RBAN");
#endif

		// RBAN collision customization
		Implementation->Collisions.DisableHandles();

		FCollisionDetectorSettings DetectorSettings = Implementation->Collisions.GetDetectorSettings();
		DetectorSettings.bFilteringEnabled = false;
		DetectorSettings.bAllowManifoldReuse = false;
		DetectorSettings.bDeferNarrowPhase = (ChaosImmediate_Collision_DeferNarrowPhase != 0);
		DetectorSettings.bAllowCCD = false;
		DetectorSettings.bAllowMACD = false;
		Implementation->Collisions.SetDetectorSettings(DetectorSettings);
	}

	FSimulation::~FSimulation()
	{
		using namespace Chaos;

		// We trace events like particles destruction, so we need to ensure there is a valid context
		// during solver destruction so these events can be traced properly.
		CVD_SCOPE_CONTEXT(CVDContextData)

		// NOTE: Particles now hold a list of all the constraints that reference them, but when
		// we delete a particle, we do not notify the constraints. When we destroy constarints
		// it tries to remove itself from the particle's list, so we must destroy the
		// constraint first.

		for (FJointHandle* JointHandle : Implementation->JointHandles)
		{
			delete JointHandle;
		}
		Implementation->JointHandles.Empty();

		for (FActorHandle* ActorHandle : Implementation->ActorHandles)
		{
			delete ActorHandle;
		}
		Implementation->ActorHandles.Empty();
	}

	int32 FSimulation::NumActors() const
	{
		return Implementation->ActorHandles.Num();
	}

	FActorHandle* FSimulation::GetActorHandle(int32 ActorHandleIndex)
	{
		return Implementation->ActorHandles[ActorHandleIndex];
	}
	
	const FActorHandle* FSimulation::GetActorHandle(int32 ActorHandleIndex) const
	{
		return Implementation->ActorHandles[ActorHandleIndex];
	}

	FActorHandle* FSimulation::CreateStaticActor(FBodyInstance* BodyInstance)
	{
		return CreateActor(ImmediatePhysics::MakeActorSetup(EActorType::StaticActor, BodyInstance, BodyInstance->GetUnrealWorldTransform()));
	}

	FActorHandle* FSimulation::CreateKinematicActor(FBodyInstance* BodyInstance, const FTransform& Transform)
	{
		return CreateActor(ImmediatePhysics::MakeActorSetup(EActorType::KinematicActor, BodyInstance, Transform));
	}

	FActorHandle* FSimulation::CreateDynamicActor(FBodyInstance* BodyInstance, const FTransform& Transform)
	{
		return CreateActor(ImmediatePhysics::MakeActorSetup(EActorType::DynamicActor, BodyInstance, Transform));
	}

	FActorHandle* FSimulation::CreateActor(EActorType ActorType, FBodyInstance* BodyInstance, const FTransform& Transform)
	{
		return CreateActor(ImmediatePhysics::MakeActorSetup(ActorType, BodyInstance, Transform));
	}

	FActorHandle* FSimulation::CreateActor(FActorSetup&& ActorSetup)
	{
		FActorHandle* ActorHandle = new FActorHandle(Implementation->Particles, Implementation->ParticlePrevXs, Implementation->ParticlePrevRs, MoveTemp(ActorSetup));
		int ActorIndex = Implementation->ActorHandles.Add(ActorHandle);

		ActorHandle->GetParticle()->AuxilaryValue(Implementation->ParticleMaterials) = MakeSerializable(ActorSetup.Material);
		ActorHandle->GetParticle()->AuxilaryValue(Implementation->PerParticleMaterials) = MoveTemp(ActorSetup.Material);
		ActorHandle->GetParticle()->AuxilaryValue(Implementation->CollidedParticles) = false;
		ActorHandle->GetParticle()->AuxilaryValue(Implementation->ParticlePrevXs) = ActorHandle->GetParticle()->GetX();
		ActorHandle->GetParticle()->AuxilaryValue(Implementation->ParticlePrevRs) = ActorHandle->GetParticle()->GetR();

		Implementation->bActorsDirty = true;

		return ActorHandle;
	}


	void FSimulation::DestroyActor(FActorHandle* ActorHandle)
	{
		// @todo(ccaulfield): FActorHandle could remember its index to optimize this

		RemoveFromCollidingPairs(ActorHandle);
        DestroyActorCollisions(ActorHandle);

		// If any joints reference the particle, we must destroy them
		TArray<FJointHandle*> ActorJointHandles;
		for (FJointHandle* JointHandle : Implementation->JointHandles)
		{
			if (JointHandle != nullptr)
			{
				if ((JointHandle->GetActorHandles()[0] == ActorHandle) || (JointHandle->GetActorHandles()[1] == ActorHandle))
				{
					ActorJointHandles.Add(JointHandle);
				}
			}
		}
		for (FJointHandle* JointHandle : ActorJointHandles)
		{
			DestroyJoint(JointHandle);
		}
		ActorJointHandles.Empty();


		int32 Index = Implementation->ActorHandles.Remove(ActorHandle);
		delete ActorHandle;

		Implementation->bActorsDirty = true;
	}

    void FSimulation::DestroyActorCollisions(FActorHandle* ActorHandle)
    {
        Implementation->Collisions.GetConstraintAllocator().RemoveParticle(ActorHandle->GetParticle());
    }

	void FSimulation::SetIsKinematic(FActorHandle* ActorHandle, bool bKinematic)
	{
		bool bWasKinematic = ActorHandle->GetIsKinematic();
		if (bKinematic != bWasKinematic)
		{
			ActorHandle->SetIsKinematic(bKinematic);
			Implementation->bActorsDirty = true;
			// Force constraint sorting to be updated
			Implementation->Joints.LinearConstraints.MarkJointsDirty();
			Implementation->Joints.NonLinearConstraints.MarkJointsDirty();
		}
	}

	void FSimulation::SetEnabled(FActorHandle* ActorHandle, bool bEnable)
	{
		bool bWasEnabled = ActorHandle->GetEnabled();
		if (bEnable != bWasEnabled)
		{
			ActorHandle->SetEnabled(bEnable);
			Implementation->bActorsDirty = true;
			// Force constraint sorting to be updated
			Implementation->Joints.LinearConstraints.MarkJointsDirty();
			Implementation->Joints.NonLinearConstraints.MarkJointsDirty();
		}
	}

	void FSimulation::SetHasCollision(FActorHandle* ActorHandle, bool bCollision)
	{
		bool bWasCollision = ActorHandle->GetHasCollision();
		if (bCollision != bWasCollision)
		{
			ActorHandle->SetHasCollision(bCollision);
			Implementation->bActorsDirty = true;
		}
	}

	FJointHandle* FSimulation::CreateJoint(FConstraintInstance* ConstraintInstance, FActorHandle* Body1, FActorHandle* Body2)
	{
		return CreateJoint(ImmediatePhysics::MakeJointSetup(ConstraintInstance, Body1, Body2));
	}

	FJointHandle* FSimulation::CreateJoint(const FJointSetup& JointSetup)
	{
		if (JointSetup.ConstraintSettings.bUseLinearSolver)
		{
			FJointHandle* JointHandle = new FJointHandle(&(Implementation->Joints.LinearConstraints), JointSetup);
			Implementation->JointHandles.Add(JointHandle);
			return JointHandle;
		}
		else
		{
			FJointHandle* JointHandle = new FJointHandle(&(Implementation->Joints.NonLinearConstraints), JointSetup);
			Implementation->JointHandles.Add(JointHandle);
			return JointHandle;
		}
	}

	void FSimulation::DestroyJoint(FJointHandle* JointHandle)
	{
		// @todo(ccaulfield): FJointHandle could remember its index to optimize this
		Implementation->JointHandles.Remove(JointHandle);
		delete JointHandle;
	}

	void FSimulation::SetNumActiveBodies(int32 InNumActiveActorHandles, TArray<int32> ActiveBodyIndices)
	{
		if (InNumActiveActorHandles == Implementation->NumActiveDynamicActorHandles && ChaosImmediate_DisableInactiveByIndex == 0)
		{
			return;
		}

		// @todo(ccaulfield): can be optimized, but I think we end up with kinematic at the start and statics at the end of the 
		// list - maybe split them up or put kinematics at the end (in a way that does not impact particle order).
		Implementation->NumActiveDynamicActorHandles = 0;
		for (int32 ActorHandleIndex = 0; ActorHandleIndex < Implementation->ActorHandles.Num(); ++ActorHandleIndex)
		{
			FActorHandle* Handle = GetActorHandle(ActorHandleIndex);
			if (Handle->GetIsKinematic())
			{
				continue;
			}
			if (ChaosImmediate_DisableInactiveByIndex != 0)
			{
				if (ActiveBodyIndices.Contains(ActorHandleIndex))
				{
					Handle->SetEnabled(true);
					++Implementation->NumActiveDynamicActorHandles;
				}
				else
				{
					Handle->SetEnabled(false);
				}
			}
			else
			{
				if (Implementation->NumActiveDynamicActorHandles < InNumActiveActorHandles)
				{
					Handle->SetEnabled(true);
					++Implementation->NumActiveDynamicActorHandles;
				}
				else
				{
					Handle->SetEnabled(false);
				}
			}
		}
		Implementation->bActorsDirty = true;
	}

	// Currently sets up potential collision with ActorHandle and all dynamics
	void FSimulation::AddToCollidingPairs(FActorHandle* ActorHandle)
	{
		bool bCouldBeDynamic = ActorHandle->CouldBeDynamic();
		using namespace Chaos;
		FGeometryParticleHandle* Particle0 = ActorHandle->GetParticle();
		for (FActorHandle* OtherActorHandle : Implementation->ActorHandles)
		{
			FGeometryParticleHandle* Particle1 = OtherActorHandle->GetParticle();
			if ((OtherActorHandle != ActorHandle) && (bCouldBeDynamic || OtherActorHandle->CouldBeDynamic()))
			{
				Implementation->PotentiallyCollidingPairs.Emplace(FParticlePair(Particle0, Particle1));
			}
		}
		Implementation->bActorsDirty = true;
	}

	void FSimulation::RemoveFromCollidingPairs(FActorHandle* ActorHandle)
	{
		using namespace Chaos;
		for (FParticlePair& ParticlePair : Implementation->PotentiallyCollidingPairs)
		{
			if ((ParticlePair[0] == ActorHandle->GetParticle()) || (ParticlePair[1] == ActorHandle->GetParticle()))
			{
				ParticlePair[0] = nullptr;
				ParticlePair[1] = nullptr;
			}
		}
		Implementation->bActorsDirty = true;
	}

	void FSimulation::PackCollidingPairs()
	{
		int32 NextValidPairIndex = 0;
		for (int32 PairIndex = 0; PairIndex < Implementation->PotentiallyCollidingPairs.Num(); ++PairIndex)
		{
			if (Implementation->PotentiallyCollidingPairs[PairIndex][0] == nullptr)
			{
				NextValidPairIndex = FMath::Max(NextValidPairIndex, PairIndex + 1);
				while ((NextValidPairIndex < Implementation->PotentiallyCollidingPairs.Num()) && (Implementation->PotentiallyCollidingPairs[NextValidPairIndex][0] == nullptr))
				{
					++NextValidPairIndex;
				}
				if (NextValidPairIndex >= Implementation->PotentiallyCollidingPairs.Num())
				{
					Implementation->PotentiallyCollidingPairs.SetNum(PairIndex);
					break;
				}
				Implementation->PotentiallyCollidingPairs[PairIndex] = Implementation->PotentiallyCollidingPairs[NextValidPairIndex];
				Implementation->PotentiallyCollidingPairs[NextValidPairIndex][0] = nullptr;
				Implementation->PotentiallyCollidingPairs[NextValidPairIndex][1] = nullptr;
				++NextValidPairIndex;
			}
		}
	}

	void FSimulation::SetIgnoreCollisionPairTable(const TArray<FIgnorePair>& InIgnoreCollisionPairTable)
	{
		using namespace Chaos;

		Implementation->IgnoreCollisionParticlePairTable.Reset();
		for (const FIgnorePair& IgnorePair : InIgnoreCollisionPairTable)
		{
			if(!IgnorePair.A || !IgnorePair.B)
			{
				continue;
			}

			const FGeometryParticleHandle* ParticleA = IgnorePair.A->GetParticle();
			const FGeometryParticleHandle* ParticleB = IgnorePair.B->GetParticle();

			Implementation->IgnoreCollisionParticlePairTable.FindOrAdd(ParticleA).Add(ParticleB); 
			Implementation->IgnoreCollisionParticlePairTable.FindOrAdd(ParticleB).Add(ParticleA);
		}

		Implementation->PotentiallyCollidingPairs.Empty();
		int NumActorHandles = Implementation->ActorHandles.Num();
		for (int ActorHandleIndex0 = 0; ActorHandleIndex0 < NumActorHandles; ++ActorHandleIndex0)
		{
			FActorHandle* ActorHandle0 = Implementation->ActorHandles[ActorHandleIndex0];
			FGeometryParticleHandle* Particle0 = ActorHandle0->GetParticle();
			//FPBDRigidParticleHandle* Rigid0 = Particle0->CastToRigidParticle();
			//if (Rigid0 && (Rigid0->ObjectState() == EObjectStateType::Dynamic))
			{
				for (int ActorHandleIndex1 = ActorHandleIndex0 + 1; ActorHandleIndex1 < NumActorHandles; ++ActorHandleIndex1)
				{
					FActorHandle* ActorHandle1 = Implementation->ActorHandles[ActorHandleIndex1];
					FGeometryParticleHandle* Particle1 = ActorHandle1->GetParticle();

					const TSet<const typename Chaos::FGeometryParticleHandle*>* Particle0IgnoreSet = Implementation->IgnoreCollisionParticlePairTable.Find(Particle0);
					bool bIgnoreActorHandlePair = (Particle0IgnoreSet != nullptr) && Particle0IgnoreSet->Contains(Particle1);
					if (!bIgnoreActorHandlePair)
					{
						Implementation->PotentiallyCollidingPairs.Emplace(FParticlePair(Particle0, Particle1));
					}
				}
			}
		}

		Implementation->bActorsDirty = true;
	}

	void FSimulation::SetIgnoreCollisionActors(const TArray<FActorHandle*>& InIgnoreCollisionActors)
	{
		using namespace Chaos;

		for (FActorHandle* ActorHandle : InIgnoreCollisionActors)
		{
			FPBDRigidParticleHandle* Particle = ActorHandle->GetParticle()->CastToRigidParticle();
			if (Particle != nullptr && Particle->ObjectState() == EObjectStateType::Dynamic)
			{
				Particle->SetCollisionGroup(INDEX_NONE);
			}
		}

		Implementation->bActorsDirty = true;
	}

	void FSimulation::UpdateActivePotentiallyCollidingPairs()
	{
		using namespace Chaos;

		Implementation->ActivePotentiallyCollidingPairs.Reset();
		for (const FParticlePair& ParticlePair : Implementation->PotentiallyCollidingPairs)
		{
			if ((ParticlePair[0] != nullptr) && (ParticlePair[1] != nullptr))
			{
				bool bAnyDisabled = FGenericParticleHandle(ParticlePair[0])->Disabled() || FGenericParticleHandle(ParticlePair[1])->Disabled();
				bool bAnyDynamic = FGenericParticleHandle(ParticlePair[0])->IsDynamic() || FGenericParticleHandle(ParticlePair[1])->IsDynamic();
				bool bBothCollide = ParticlePair[0]->HasCollision() && ParticlePair[1]->HasCollision();
				if (bBothCollide && bAnyDynamic && !bAnyDisabled)
				{
					Implementation->ActivePotentiallyCollidingPairs.Add(ParticlePair);
				}
			}
		}
	}

	void FSimulation::EnableDisableJoints()
	{
		for (FJointHandle* Joint : Implementation->JointHandles)
		{
			const bool bActorEnabled0 = Joint->GetActorHandles()[0]->GetEnabled();
			const bool bActorEnabled1 = Joint->GetActorHandles()[1]->GetEnabled();
			const bool bJointEnabled = bActorEnabled0 && bActorEnabled1;
			Joint->GetConstraint()->SetConstraintEnabled(bJointEnabled);
		}
	}


	void FSimulation::InitSimulationSpace(
		const FTransform& Transform)
	{
		UpdateSimulationSpace(Transform, FVector::ZeroVector, FVector::ZeroVector, FVector::ZeroVector, FVector::ZeroVector);
	}

	void FSimulation::UpdateSimulationSpace(
		const FTransform& Transform,
		const FVector& LinearVel,
		const FVector& AngularVel,
		const FVector& LinearAcc,
		const FVector& AngularAcc)
	{
		Implementation->SimulationSpaceTransform = Transform;
		Implementation->SimulationSpace.LinearAcceleration = LinearAcc;
		Implementation->SimulationSpace.AngularAcceleration = AngularAcc;
		Implementation->SimulationSpace.LinearVelocity = LinearVel;
		Implementation->SimulationSpace.AngularVelocity = AngularVel;
	}

	void FSimulation::SetSimulationSpaceSettings(const bool bEnabled, const FReal DampingAlpha, const FVector& ExternalLinearEtherDrag)
	{
		using namespace Chaos;

		FSimulationSpaceSettings& SimSpaceSettings = Implementation->Evolution.GetSimulationSpaceSettings();
		SimSpaceSettings.bEnabled = bEnabled;
		SimSpaceSettings.LinearVelocityAlpha = DampingAlpha;
		SimSpaceSettings.AngularVelocityAlpha = DampingAlpha;
		SimSpaceSettings.ExternalLinearEtherDrag = ExternalLinearEtherDrag;
		SimSpaceSettings.CentrifugalAlpha = ChaosImmediate_Evolution_SimSpaceCentrifugalAlpha;
		SimSpaceSettings.CoriolisAlpha = ChaosImmediate_Evolution_SimSpaceCoriolisAlpha;
		SimSpaceSettings.EulerAlpha = ChaosImmediate_Evolution_SimSpaceEulerAlpha;
	}

	void FSimulation::SetSimulationSpaceSettings(const Chaos::FSimulationSpaceSettings& SimulationSpaceSettings)
	{
		Implementation->Evolution.SetSimulationSpaceSettings(SimulationSpaceSettings);
	}

	const Chaos::FSimulationSpaceSettings& FSimulation::GetSimulationSpaceSettings() const
	{
		return Implementation->Evolution.GetSimulationSpaceSettings();
	}

	void FSimulation::SetSolverSettings(const FReal FixedDt, const FReal CullDistance, const FReal MaxDepenetrationVelocity, const int32 UseLinearJointSolver, const int32 PositionIts, const int32 VelocityIts, const int32 ProjectionIts, const int32 bUseManifoilds)
	{
		if (FixedDt >= FReal(0))
		{
			Implementation->FixedStepTime = FixedDt;
		}

		if (CullDistance >= FReal(0))
		{
			Implementation->Collisions.SetCullDistance(CullDistance);
		}

		if (MaxDepenetrationVelocity >= FReal(0))
		{
			Implementation->Collisions.SetMaxPushOutVelocity(MaxDepenetrationVelocity);
		}

		if (PositionIts >= 0)
		{
			Implementation->Evolution.SetNumPositionIterations(PositionIts);
		}

		if (VelocityIts >= 0)
		{
			Implementation->Evolution.SetNumVelocityIterations(VelocityIts);
		}

		if (ProjectionIts >= 0)
		{
			Implementation->Evolution.SetNumProjectionIterations(ProjectionIts);
		}

		if (UseLinearJointSolver >= 0)
		{
			if (UseLinearJointSolver != 0)
			{
				if (Implementation->Joints.NonLinearConstraints.NumConstraints() > 0)
				{
					Implementation->Joints.MergeNonLinearConstraintsIntoLinearConstraints();
				}
			}
			else
			{
				if (Implementation->Joints.LinearConstraints.NumConstraints() > 0)
				{
					Implementation->Joints.MergeLinearConstraintsIntoNonLinearConstraints();
				}
			}
			Implementation->Joints.LinearConstraints.SetUseSimd(bChaosImmediate_UseSimdForLinearSolver); 
		}

		if (bUseManifoilds >= 0)
		{
			Implementation->Collisions.SetAllowManifolds(bUseManifoilds != 0);
		}
	}

	void FSimulation::SetMaxNumRollingAverageStepTimes(const int32 MaxNumRollingAverageStepTimes)
	{
		if (MaxNumRollingAverageStepTimes >= 0)
		{
			Implementation->MaxNumRollingAverageStepTimes = MaxNumRollingAverageStepTimes;
		}
	}

	void FSimulation::SetUseMinStepTime(bool bUse)
	{
		Implementation->bUseMinStepTime = bUse;
	}

	void FSimulation::SetUseFixedStepTolerance(bool bUse)
	{
		Implementation->bUseFixedStepTolerance = bUse;
	}


	void FSimulation::DebugDraw()
	{
#if CHAOS_DEBUG_DRAW
		ChaosDD::Private::FChaosDDScopeTimelineContext DDContext(DDSimulationTimeline, 0.0, 0.0);
#endif

		DebugDrawStaticParticles();
		DebugDrawKinematicParticles();
		DebugDrawDynamicParticles();
		DebugDrawConstraints();
		DebugDrawSimulationSpace();
	}

	FReal FSimulation::UpdateStepTime(const FReal DeltaTime, const FReal MaxStepTime)
	{
		if (Implementation->NumRollingAverageStepTimes > 1)
		{
			Implementation->RollingAverageStepTime = Implementation->RollingAverageStepTime + (DeltaTime - Implementation->RollingAverageStepTime) / Implementation->NumRollingAverageStepTimes;
			Implementation->RollingAverageStepTime = FMath::Min(Implementation->RollingAverageStepTime, MaxStepTime);
			Implementation->NumRollingAverageStepTimes = FMath::Min(Implementation->NumRollingAverageStepTimes + 1, Implementation->MaxNumRollingAverageStepTimes);
		}
		else
		{
			// Adjust things so that if we go back to smoothing the step time, we do so gracefully
			Implementation->NumRollingAverageStepTimes = Implementation->MaxNumRollingAverageStepTimes;
			Implementation->RollingAverageStepTime = FMath::Min(DeltaTime, MaxStepTime);
		}
		return Implementation->RollingAverageStepTime;
	}

	void FSimulation::SetRewindVelocities(bool bRewindVelocities)
	{
		Implementation->Evolution.SetRewindVelocities(bRewindVelocities);
	}

	void FSimulation::Simulate(
		FReal InDeltaTime, FReal MaxStepTime, int32 MaxSubSteps, const FVector& InGravity,
		Chaos::FPBDJointSolverSettings* JointSolverSettings, 
		Chaos::FPBDCollisionSolverSettings* CollisionSolverSettings,
		Chaos::FCollisionDetectorSettings* CollisionDetectorSettings)
	{
		SCOPE_CYCLE_COUNTER(STAT_ImmediateSimulate_Chaos);
		CVD_SCOPE_TRACE_SOLVER_FRAME(FSimulation, *this);
		CVD_TRACE_SOLVER_SIMULATION_SPACE(Implementation->SimulationSpaceTransform);

		using namespace Chaos;

		// Reject DeltaTime outliers
		const FReal DeltaTime = FMath::Min(InDeltaTime, MaxStepTime * MaxSubSteps);

		// Update rolling average step time - we want a smooth step time from frame-to-frame that is roughly the target frame rate.
		// @todo(ccaulfield): decouple sim and game delta times and simulate ahead. Add extrapolation of kinematic targets, and interpolation of physics results.
		FReal StepTime = UpdateStepTime(DeltaTime, MaxStepTime);

		// Calculate number of steps to run
		int32 NumSteps = FMath::Clamp(FMath::RoundToInt32(DeltaTime / StepTime), 1, MaxSubSteps);

		// These are overrides, so only apply if > 0. 
		SetSolverSettings(
			ChaosImmediate_Evolution_FixedStepTime,
			ChaosImmediate_Collision_CullDistance,
			ChaosImmediate_Collision_MaxDepenetrationVelocity,
			ChaosImmediate_Joint_UseLinearSolver,
			ChaosImmediate_Evolution_PositionIterations,
			ChaosImmediate_Evolution_VelocityIterations,
			ChaosImmediate_Evolution_ProjectionIterations,
			ChaosImmediate_Collision_UseManifolds);
		SetMaxNumRollingAverageStepTimes(ChaosImmediate_Evolution_DeltaTimeCount);

		if (JointSolverSettings)
		{
			Implementation->Joints.LinearConstraints.SetSettings(*JointSolverSettings);
			Implementation->Joints.NonLinearConstraints.SetSettings(*JointSolverSettings);
		}
		else
		{
			FPBDJointSolverSettings JointsSettings = Implementation->Joints.LinearConstraints.GetSettings();
			JointsSettings.SwingTwistAngleTolerance = ChaosImmediate_Joint_SwingTwistAngleTolerance;
			JointsSettings.PositionTolerance = ChaosImmediate_Joint_PositionTolerance;
			JointsSettings.AngleTolerance = ChaosImmediate_Joint_AngleTolerance;
			JointsSettings.MinParentMassRatio = ChaosImmediate_Joint_MinParentMassRatio;
			JointsSettings.MaxInertiaRatio = ChaosImmediate_Joint_MaxInertiaRatio;
			JointsSettings.bSolvePositionLast = ChaosImmediate_Joint_SolvePositionLast != 0;
			JointsSettings.bUsePositionBasedDrives = ChaosImmediate_Joint_UsePositionBasedDrives != 0;
			JointsSettings.bEnableTwistLimits = ChaosImmediate_Joint_EnableTwistLimits != 0;
			JointsSettings.bEnableSwingLimits = ChaosImmediate_Joint_EnableSwingLimits != 0;
			JointsSettings.bEnableDrives = ChaosImmediate_Joint_EnableDrives != 0;
			JointsSettings.LinearStiffnessOverride = ChaosImmediate_Joint_Stiffness;
			JointsSettings.TwistStiffnessOverride = ChaosImmediate_Joint_Stiffness;
			JointsSettings.SwingStiffnessOverride = ChaosImmediate_Joint_Stiffness;
			JointsSettings.LinearProjectionOverride = ChaosImmediate_Joint_LinearProjection;
			JointsSettings.AngularProjectionOverride = ChaosImmediate_Joint_AngularProjection;
			JointsSettings.ShockPropagationOverride = ChaosImmediate_Joint_ShockPropagation;
			JointsSettings.SoftLinearStiffnessOverride = ChaosImmediate_Joint_SoftLinearStiffness;
			JointsSettings.SoftTwistStiffnessOverride = ChaosImmediate_Joint_SoftTwistStiffness;
			JointsSettings.SoftTwistDampingOverride = ChaosImmediate_Joint_SoftTwistDamping;
			JointsSettings.SoftSwingStiffnessOverride = ChaosImmediate_Joint_SoftSwingStiffness;
			JointsSettings.SoftSwingDampingOverride = ChaosImmediate_Joint_SoftSwingDamping;
			JointsSettings.LinearDriveStiffnessOverride = ChaosImmediate_Joint_LinearDriveStiffness;
			JointsSettings.LinearDriveDampingOverride = ChaosImmediate_Joint_LinearDriveDamping;
			JointsSettings.AngularDriveStiffnessOverride = ChaosImmediate_Joint_AngularDriveStiffness;
			JointsSettings.AngularDriveDampingOverride = ChaosImmediate_Joint_AngularDriveDamping;
			JointsSettings.NumShockPropagationIterations = ChaosImmediate_Joint_NumShockPropagationIterations;
			JointsSettings.bSortEnabled = true;
			Implementation->Joints.LinearConstraints.SetSettings(JointsSettings);
			Implementation->Joints.NonLinearConstraints.SetSettings(JointsSettings);
		}

		if (CollisionSolverSettings)
		{
			Implementation->Collisions.SetSolverSettings(*CollisionSolverSettings);
		}
		else
		{
			Implementation->Collisions.SetPositionFrictionIterations(ChaosImmediate_Collision_NumPositionFrictionIterations);
			Implementation->Collisions.SetVelocityFrictionIterations(ChaosImmediate_Collision_NumVelocityFrictionIterations);
			Implementation->Collisions.SetPositionShockPropagationIterations(ChaosImmediate_Collision_NumPositionShockPropagationIterations);
			Implementation->Collisions.SetVelocityShockPropagationIterations(ChaosImmediate_Collision_NumVelocityShockPropagationIterations);

			Implementation->Collisions.SetRestitutionEnabled(ChaosImmediate_Collision_RestitutionEnabled != 0);
			Implementation->Collisions.SetRestitutionThreshold(ChaosImmediate_Collision_RestitutionThresholdMultiplier * InGravity.Size());
			Implementation->Collisions.SetCollisionsEnabled(ChaosImmediate_Collision_Enabled != 0);

			Implementation->Evolution.SetConstraintContainerPriority(Implementation->Collisions.GetContainerId(), ChaosImmediate_Collision_Priority);
		}

		if (CollisionDetectorSettings)
		{
			Implementation->Collisions.SetDetectorSettings(*CollisionDetectorSettings);
		}
		else
		{
			FCollisionDetectorSettings DetectorSettings = Implementation->Collisions.GetDetectorSettings();
			DetectorSettings.bAllowManifoldReuse = false;
			DetectorSettings.bDeferNarrowPhase = (ChaosImmediate_Collision_DeferNarrowPhase != 0);;
			Implementation->Collisions.SetDetectorSettings(DetectorSettings);
		}

		if (ChaosImmediate_Evolution_StepTime > 0)
		{
			StepTime = ChaosImmediate_Evolution_StepTime;
		}
		if (ChaosImmediate_Evolution_NumSteps > 0)
		{
			NumSteps = ChaosImmediate_Evolution_NumSteps;
		}

		// Fixed timestep mode DT (Only used if > 0.0f)
		FReal FixedStepTime = Implementation->FixedStepTime;

		// Use fixed step mode anyway if StepTime is too low
		// This can prevent collision push resolution introducing large velocities at small DTs
		if (Implementation->bUseMinStepTime && (FixedStepTime <= 0.0f) && (StepTime < ChaosImmediate_Evolution_MinStepTime))
		{
			FixedStepTime = ChaosImmediate_Evolution_MinStepTime;
		}

		// If using FixedStep mode, calculate the number of steps and how far to rewind (if at all)
		FReal RewindTime = 0.0f;
		if (FixedStepTime > 0)
		{
			StepTime = FixedStepTime;
			NumSteps = FMath::FloorToInt(DeltaTime / StepTime);
			FReal RemainderTime = DeltaTime - NumSteps * StepTime;
			if (!Implementation->bUseFixedStepTolerance || RemainderTime > ChaosImmediate_Evolution_FixedStepTolerance * StepTime)
			{
				++NumSteps;
				RewindTime = StepTime - RemainderTime;
			}
			NumSteps = FMath::Max(1, NumSteps);
		}

		// Handle new or deleted particles
		if (Implementation->bActorsDirty)
		{
			PackCollidingPairs();
			UpdateActivePotentiallyCollidingPairs();
			EnableDisableJoints();
			Implementation->bActorsDirty = false;
		}

		UpdateInertiaConditioning(InGravity);

		UE_LOGF(LogChaosJoint, Verbose, "Simulate Dt = %f Steps %d x %f (Rewind %f)", DeltaTime, NumSteps, StepTime, RewindTime);
		Implementation->Evolution.SetGravity(InGravity);
		Implementation->Evolution.SetSimulationSpace(Implementation->SimulationSpace);
		Implementation->Evolution.Advance(StepTime, NumSteps, RewindTime);

		UpdateStatCounters();

#if CHAOS_DEBUG_DRAW
		if (bChaosImmediate_DebugDrawOnSimulate)
		{
			DebugDraw();
		}
#endif
	}

	// @todo(chaos): Move implementation to MinEvolution
	void FSimulation::UpdateInertiaConditioning(const FVector& Gravity)
	{
		using namespace Chaos;

		SCOPE_CYCLE_COUNTER(STAT_ImmediateSimulate_Chaos_InertiaConditioning);

		// Linear and Non-linear joint solvers have a separate cvar to enable inertia conditioning
		const bool bUseInertiaConditioning = Implementation->Joints.LinearConstraints.NumConstraints() > 0 ? bChaosImmediate_LinearInertiaConditioningEnabled : bChaosImmediate_NonLinearInertiaConditioningEnabled;

		if (bUseInertiaConditioning)
		{
			// The maximum contribution to error correction from rotation
			const FRealSingle MaxRotationRatio = ChaosImmediate_InertiaConditioningRotationRatio;

			// The error distance that the constraint correction must be stable for
			// @todo(chaos): should probably be tied to constraint teleport threshold?
			const FRealSingle MaxDistance = ChaosImmediate_InertiaConditioningDistance;

			// A limit on the relative sizes of the inertia components (inverse)
			const FRealSingle MaxInvInertiaComponentRatio = ChaosImmediate_MaxInvInertiaComponentRatio;

			// @todo(chaos): keep track of changes to the flags so we don't need to loop every tick (maybe check it in Integrate)
			for (FActorHandle* Actor : Implementation->ActorHandles)
			{
				FGenericParticleHandle Particle = Actor->GetParticle();
				if (Particle->InertiaConditioningDirty())
				{
					const bool bWantInertiaConditioning = Particle->IsDynamic() && Particle->InertiaConditioningEnabled();
					if (bWantInertiaConditioning)
					{
						// Calculate the extents of all the constraints on the body (in CoM space)
						// @chaos(todo): We need a way to iterate over joints attached to a specific actor
						const FVec3 CollisionExtents = FReal(0.5) * Particle->LocalBounds().InverseTransformedAABB(Actor->GetLocalCoMTransform()).Extents();
						FVec3 ConstraintExtents = FVec3(0);
						for (const FJointHandle* Joint : Implementation->JointHandles)
						{
							if (!FPBDJointUtilities::GetLinearIsFree(Joint->GetConstraint()->GetSettings()))
							{
								const Chaos::TVec2<const FActorHandle*>& JointActors = Joint->GetActorHandles();
								FVec3 JointArm = FVec3(0);
								if (JointActors[0] == Actor)
								{
									JointArm = Actor->GetLocalCoMTransform().InverseTransformPositionNoScale(Joint->GetConstraint()->GetSettings().ConnectorTransforms[0].GetTranslation());
								}
								else if (JointActors[1] == Actor)
								{
									JointArm = Actor->GetLocalCoMTransform().InverseTransformPositionNoScale(Joint->GetConstraint()->GetSettings().ConnectorTransforms[1].GetTranslation());
								}
								ConstraintExtents = FVec3::Max(ConstraintExtents, JointArm.GetAbs());
							}
						}

						const FVec3f CoMExtents = FVec3f::Max(FVec3f(CollisionExtents), FVec3f(ConstraintExtents));
						const FVec3f ActorInertiaConditioning = CalculateInertiaConditioning(FRealSingle(Actor->GetInverseMass()), FVec3f(Actor->GetInverseInertia()), CoMExtents, MaxDistance, MaxRotationRatio, MaxInvInertiaComponentRatio);
				
						Particle->SetInvIConditioning(ActorInertiaConditioning);
					}
					else
					{
						Particle->SetInvIConditioning(FVec3f(1));
					}
					Particle->ClearInertiaConditioningDirty();
				}
			}
		}
	}

	void FSimulation::UpdateStatCounters()
	{
		INC_DWORD_STAT_BY(STAT_ImmediateSimulate_ChaosCounter_NumBodies, Implementation->Particles.GetNonDisabledView().Num());
		INC_DWORD_STAT_BY(STAT_ImmediateSimulate_ChaosCounter_NumDynamicBodies, Implementation->Particles.GetNonDisabledDynamicView().Num());
		INC_DWORD_STAT_BY(STAT_ImmediateSimulate_ChaosCounter_NumKinematicBodies, Implementation->Particles.GetActiveKinematicParticlesView().Num());
		INC_DWORD_STAT_BY(STAT_ImmediateSimulate_ChaosCounter_NumContacts, Implementation->Collisions.NumConstraints());
		INC_DWORD_STAT_BY(STAT_ImmediateSimulate_ChaosCounter_NumJoints, Implementation->Joints.NumConstraints());
	}
	void FSimulation::DebugDrawStaticParticles()
	{
#if CHAOS_DEBUG_DRAW
		using namespace Chaos;
		if (FDebugDrawQueue::IsDebugDrawingEnabled() && bChaosImmediate_DebugDrawShowStatics)
		{
			if (bChaosImmediate_DebugDrawParticles)
			{
				DebugDraw::DrawParticleTransforms(Implementation->SimulationSpaceTransform, Implementation->Particles.GetActiveStaticParticlesView(), &ChaosImmPhysDebugDebugDrawSettings);
			}
			if (bChaosImmediate_DebugDrawShapes)
			{
				DebugDraw::DrawParticleShapes(Implementation->SimulationSpaceTransform, Implementation->Particles.GetActiveStaticParticlesView(), 1.0f, &ChaosImmPhysDebugDebugDrawSettings);
			}
			if (bChaosImmediate_DebugDrawBounds)
			{
				DebugDraw::DrawParticleBounds(Implementation->SimulationSpaceTransform, Implementation->Particles.GetActiveStaticParticlesView(), 0.0f, &ChaosImmPhysDebugDebugDrawSettings);
			}
		}
#endif
	}

	void FSimulation::DebugDrawKinematicParticles()
	{
#if CHAOS_DEBUG_DRAW
		using namespace Chaos;
		if (FDebugDrawQueue::IsDebugDrawingEnabled() && bChaosImmediate_DebugDrawShowKinematics)
		{
			if (bChaosImmediate_DebugDrawParticles)
			{
				DebugDraw::DrawParticleTransforms(Implementation->SimulationSpaceTransform, Implementation->Particles.GetActiveKinematicParticlesView(), &ChaosImmPhysDebugDebugDrawSettings);
			}
			if (bChaosImmediate_DebugDrawShapes)
			{
				DebugDraw::DrawParticleShapes(Implementation->SimulationSpaceTransform, Implementation->Particles.GetActiveKinematicParticlesView(), 1.0f, &ChaosImmPhysDebugDebugDrawSettings);
			}
			if (bChaosImmediate_DebugDrawBounds)
			{
				DebugDraw::DrawParticleBounds(Implementation->SimulationSpaceTransform, Implementation->Particles.GetActiveKinematicParticlesView(), 0.0f, &ChaosImmPhysDebugDebugDrawSettings);
			}
		}
#endif
	}

	void FSimulation::DebugDrawDynamicParticles()
	{
#if CHAOS_DEBUG_DRAW
		using namespace Chaos;
		if (FDebugDrawQueue::IsDebugDrawingEnabled() && bChaosImmediate_DebugDrawShowDynamics)
		{
			if (bChaosImmediate_DebugDrawParticles)
			{
				DebugDraw::DrawParticleTransforms(Implementation->SimulationSpaceTransform, Implementation->Particles.GetActiveParticlesView(), &ChaosImmPhysDebugDebugDrawSettings);
			}
			if (bChaosImmediate_DebugDrawShapes)
			{
				DebugDraw::DrawParticleShapes(Implementation->SimulationSpaceTransform, Implementation->Particles.GetActiveParticlesView(), 1.0f, &ChaosImmPhysDebugDebugDrawSettings);
			}
			if (bChaosImmediate_DebugDrawBounds)
			{
				DebugDraw::DrawParticleBounds(Implementation->SimulationSpaceTransform, Implementation->Particles.GetActiveParticlesView(), 0.0f, &ChaosImmPhysDebugDebugDrawSettings);
			}
		}
#endif
	}

	const Chaos::FPBDCollisionSolverSettings& FSimulation::GetCollisionSolverSettings() const
	{
		return Implementation->Collisions.GetSolverSettings();
	}

	const Chaos::FCollisionDetectorSettings& FSimulation::GetCollisionDetectorSettings() const
	{
		return Implementation->Collisions.GetDetectorSettings();
	}

	void FSimulation::SetCollisionDetectorSettings(const Chaos::FCollisionDetectorSettings& Settings)
	{
		Implementation->Collisions.SetDetectorSettings(Settings);
	}

	Chaos::FVec3 FSimulation::FCollisionData::GetCollisionAccumulatedImpulse() const
	{
		return Collision.GetAccumulatedImpulse();
	}

	int32 FSimulation::FCollisionData::GetNumManifoldPoints() const
	{
		return Collision.NumManifoldPoints();
	}

	const void FSimulation::FCollisionData::GetManifoldPointData(
		const int32              PointIndex,
		FRealSingle&             Depth,
		Chaos::FVec3&            PlaneNormal,
		Chaos::FVec3&            PointLocation,
		Chaos::FVec3&            PlaneLocation) const
	{
		using namespace Chaos;

		Depth = Collision.GetManifoldPoint(PointIndex).InitialPhi;

		const FConstGenericParticleHandle Particle0 = Collision.GetParticle0();
		const FConstGenericParticleHandle Particle1 = Collision.GetParticle1();
		const FRigidTransform3 WorldActorTransform0 = FParticleUtilities::GetActorWorldTransform(Particle0);
		const FRigidTransform3 WorldActorTransform1 = FParticleUtilities::GetActorWorldTransform(Particle1);

		const FManifoldPoint& ManifoldPoint = Collision.GetManifoldPoint(PointIndex);
		const FManifoldPointResult& ManifoldPointResult = Collision.GetManifoldPointResult(PointIndex);

		const int32 ContactPlaneOwner = 1;
		const int32 ContactPointOwner = 1 - ContactPlaneOwner;
		const Chaos::FRigidTransform3 PlaneTransform = (ContactPlaneOwner == 0)
			? Collision.GetShapeRelativeTransform0() * WorldActorTransform0
			: Collision.GetShapeRelativeTransform1() * WorldActorTransform1;
		const Chaos::FRigidTransform3 PointTransform = (ContactPlaneOwner == 0)
			? Collision.GetShapeRelativeTransform1() * WorldActorTransform1
			: Collision.GetShapeRelativeTransform0() * WorldActorTransform0;

		PlaneNormal = PlaneTransform.TransformVectorNoScale(
			FVec3(ManifoldPoint.ContactPoint.ShapeContactNormal));
		PointLocation = PointTransform.TransformPosition(
			FVec3(ManifoldPoint.ContactPoint.ShapeContactPoints[ContactPointOwner]));
		PlaneLocation = PlaneTransform.TransformPosition(
			FVec3(ManifoldPoint.ContactPoint.ShapeContactPoints[ContactPlaneOwner]));
	}

	void FSimulation::VisitCollisions(
		TFunction<void(const FCollisionData&)> Visitor, Chaos::ECollisionVisitorFlags VisitorFlags) const
	{
		using namespace Chaos;
		const Chaos::Private::FCollisionConstraintAllocator& ConstraintAllocator = Implementation->Collisions.GetConstraintAllocator();
		ConstraintAllocator.VisitConstCollisions(
			[&](const FPBDCollisionConstraint& Collision)
			{
				Visitor(FCollisionData(Collision));
				return ECollisionVisitorResult::Continue;
			}, VisitorFlags);
	}

	void FSimulation::DebugDrawConstraints()
	{
#if CHAOS_DEBUG_DRAW
		using namespace Chaos;
		if (FDebugDrawQueue::IsDebugDrawingEnabled())
		{
			if (bChaosImmediate_DebugDrawCollisions)
			{
				DebugDraw::DrawCollisions(Implementation->SimulationSpaceTransform, Implementation->Collisions.GetConstraintAllocator(), 1.0f, &ChaosImmPhysDebugDebugDrawSettings);
			}
			if (bChaosImmediate_DebugDrawJoints)
			{
				DebugDraw::DrawJointConstraints(Implementation->SimulationSpaceTransform, Implementation->Joints.LinearConstraints, 1.0f, ChaosImmediate_DebugDrawJointFeatures, &ChaosImmPhysDebugDebugDrawSettings);
				DebugDraw::DrawJointConstraints(Implementation->SimulationSpaceTransform, Implementation->Joints.NonLinearConstraints, 1.0f, ChaosImmediate_DebugDrawJointFeatures, &ChaosImmPhysDebugDebugDrawSettings);
			}
		}
#endif
	}

	void FSimulation::DebugDrawSimulationSpace()
	{
#if CHAOS_DEBUG_DRAW
		using namespace Chaos;
		if (FDebugDrawQueue::IsDebugDrawingEnabled() && bChaosImmediate_DebugDrawSimulationSpace)
		{
			DebugDraw::DrawSimulationSpace(Implementation->SimulationSpaceTransform, Implementation->SimulationSpace, &ChaosImmPhysDebugDebugDrawSettings);
		}
#endif
	}

#if CHAOS_DEBUG_DRAW
	void FSimulation::SetDebugDrawScene(const FString& SceneName, const ChaosDD::Private::FChaosDDScenePtr& InScene)
	{
		if (InScene.IsValid() && !DDSimulationTimeline.IsValid())
		{
			DDSimulationTimeline = InScene->CreateTimeline(SceneName);
		}
	}
#endif
}
