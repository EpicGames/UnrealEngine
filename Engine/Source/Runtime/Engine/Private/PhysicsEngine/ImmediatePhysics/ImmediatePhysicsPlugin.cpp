// Copyright Epic Games, Inc. All Rights Reserved.

//#include "IImmediatePhysicsPlugin.h"
//
//class FImmediatePhysicsPlugin : public IImmediatePhysicsPlugin
//{
//
//};
//
//IMPLEMENT_MODULE(FImmediatePhysicsPlugin, ImmediatePhysics )







