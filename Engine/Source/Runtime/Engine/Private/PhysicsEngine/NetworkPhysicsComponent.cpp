// Copyright Epic Games, Inc. All Rights Reserved.

#include "Physics/NetworkPhysicsComponent.h"
#include "Components/PrimitiveComponent.h"
#include "EngineLogs.h"
#include "EngineUtils.h"
#include "GameFramework/Pawn.h"
#include "GameFramework/PlayerController.h"
#include "PBDRigidsSolver.h"
#include "Net/UnrealNetwork.h"
#include "PhysicsProxy/SingleParticlePhysicsProxy.h"
#include "Chaos/PhysicsObjectInternalInterface.h"
#include "Net/Core/PushModel/PushModel.h"
#include "Iris/ReplicationState/PropertyNetSerializerInfoRegistry.h"
#include "Iris/ReplicationState/ReplicationStateDescriptor.h"
#include "Iris/ReplicationSystem/ReplicationOperations.h"
#include "Iris/ReplicationSystem/ObjectReplicationBridge.h"
#include "Net/Iris/ReplicationSystem/EngineReplicationBridge.h"
#include "Net/Iris/ReplicationSystem/ReplicationSystemUtil.h"
#include "Iris/ReplicationSystem/NetRefHandle.h"
#include "Misc/Crc.h"

#if UE_WITH_REMOTE_OBJECT_HANDLE
#include "UObject/UObjectMigrationContext.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(NetworkPhysicsComponent)

namespace UE::Net
{
	FReplicationFragment* CreateAndRegisterNetworkPhysicsRewindDataProxyReplicationFragment(UObject* Owner, const FReplicationStateDescriptor* Descriptor, FFragmentRegistrationContext& Context)
	{
		if (FNetworkPhysicsRewindDataProxyReplicationFragment* Fragment = new FNetworkPhysicsRewindDataProxyReplicationFragment(
			Context.GetFragmentTraits() | EReplicationFragmentTraits::DeleteWithInstanceProtocol, Owner, Descriptor))
		{
			Fragment->Register(Context);
			return Fragment;
		}

		return nullptr;
	}

	FNetworkPhysicsRewindDataProxyReplicationFragment::FNetworkPhysicsRewindDataProxyReplicationFragment(EReplicationFragmentTraits InTraits, UObject* InOwner, const FReplicationStateDescriptor* InDescriptor)
		: FReplicationFragment(InTraits)
		, ReplicationStateDescriptor(InDescriptor)
		, Owner(InOwner)
	{
		// We don't want to create temporary states when applying replicated state as we want to update the replicated fields.
		Traits |= EReplicationFragmentTraits::HasPersistentTargetStateBuffer;

		if (EnumHasAnyFlags(InTraits, EReplicationFragmentTraits::CanReplicate))
		{
			SrcReplicationState = MakeUnique<FPropertyReplicationState>(InDescriptor);

			// We mark the property as dirty via overriding PollReplicatedState
			Traits |= EReplicationFragmentTraits::NeedsPoll;
		}

#if WITH_PUSH_MODEL
		if (EnumHasAnyFlags(InDescriptor->Traits, EReplicationStateTraits::HasPushBasedDirtiness))
		{
			Traits |= EReplicationFragmentTraits::HasPushBasedDirtiness;
			if (EnumHasAnyFlags(InDescriptor->Traits, EReplicationStateTraits::HasFullPushBasedDirtiness))
			{
				Traits |= EReplicationFragmentTraits::HasFullPushBasedDirtiness;
			}
		}
#endif

		if (EnumHasAnyFlags(InTraits, EReplicationFragmentTraits::CanReceive))
		{
			if (EnumHasAnyFlags(InDescriptor->Traits, EReplicationStateTraits::HasRepNotifies))
			{
				ensureMsgf(!EnumHasAnyFlags(InDescriptor->Traits, EReplicationStateTraits::KeepPreviousState), TEXT("FNetworkPhysicsRewindDataProxyReplicationFragment doesn't support OnRep calls with previous value."));
			}

			// We do custom callbacks in CallRepNotifies rather than in ApplyReplicatedState.
			Traits |= EReplicationFragmentTraits::HasRepNotifies | EReplicationFragmentTraits::NeedsLegacyCallbacks;
		}
	}

	void FNetworkPhysicsRewindDataProxyReplicationFragment::Register(FFragmentRegistrationContext& Context)
	{
		FPropertyReplicationState* ReplicationState = SrcReplicationState.Get();
		Context.RegisterReplicationFragment(this, ReplicationStateDescriptor.GetReference(), (ReplicationState ? ReplicationState->GetStateBuffer() : nullptr));
	}

	void FNetworkPhysicsRewindDataProxyReplicationFragment::ApplyReplicatedState(FReplicationStateApplyContext& ApplyContext) const
	{
		uint8* ExternalStatePointer = reinterpret_cast<uint8*>(Owner) + ReplicationStateDescriptor->MemberProperties[0]->GetOffset_ForGC();

		// Get struct instance from owner and assign owner pointer
		FNetworkPhysicsRewindDataProxy* ExternalSourceState = reinterpret_cast<FNetworkPhysicsRewindDataProxy*>(ExternalStatePointer);
		ExternalSourceState->Owner = static_cast<UNetworkPhysicsComponent*>(Owner);

		// Dequantize replicated members to external struct instance
		// Need to call the serializer directly with the appropriate arguments as the external state doesn't have a ReplicationStateHeader in front of it.
		FNetDequantizeArgs DequantizeArgs = {};
		DequantizeArgs.Source = NetSerializerValuePointer(ApplyContext.StateBufferData.RawStateBuffer);
		DequantizeArgs.Target = NetSerializerValuePointer(ExternalSourceState);
		DequantizeArgs.NetSerializerConfig = ReplicationStateDescriptor->MemberSerializerDescriptors[0].SerializerConfig;

		const FNetSerializer* Serializer = ReplicationStateDescriptor->MemberSerializerDescriptors[0].Serializer;
		Serializer->Dequantize(*ApplyContext.NetSerializationContext, DequantizeArgs);
	}

	bool FNetworkPhysicsRewindDataProxyReplicationFragment::PollReplicatedState(EReplicationFragmentPollFlags PollOption)
	{
		if (!SrcReplicationState)
		{ 
			return true;
		}

		// We can early out if we are pushbased and not dirty for polling
		const bool bPoll = EnumHasAnyFlags(PollOption, EReplicationFragmentPollFlags::PollAllState) ||
			(EnumHasAnyFlags(PollOption, EReplicationFragmentPollFlags::PollDirtyState) && (!EnumHasAnyFlags(EReplicationFragmentTraits::HasPushBasedDirtiness, Traits) || SrcReplicationState->IsDirtyForPolling()));

		if (bPoll)
		{
			return SrcReplicationState->PollPropertyReplicationState(Owner);
		}

		return SrcReplicationState->IsDirty(0);
	}

	void FNetworkPhysicsRewindDataProxyReplicationFragment::CallRepNotifies(FReplicationStateApplyContext& Context)
	{
		if (const UFunction* RepNotifyFunction = ReplicationStateDescriptor->MemberPropertyDescriptors[0].RepNotifyFunction)
		{
			if (Context.bIsInit)
			{
				const uint8* ReceivedState = Context.StateBufferData.RawStateBuffer;
				const uint8* DefaultState = ReplicationStateDescriptor->DefaultStateBuffer;

				if (FReplicationStateOperations::IsEqualQuantizedState(*Context.NetSerializationContext, ReceivedState, DefaultState, ReplicationStateDescriptor))
				{
					return;
				}
			}

			Owner->ProcessEvent(const_cast<UFunction*>(RepNotifyFunction), nullptr);
		}
	}

	void FNetworkPhysicsRewindDataProxyReplicationFragment::CollectOwner(FReplicationStateOwnerCollector* Owners) const
	{
		Owners->AddOwner(Owner);
	}
}


namespace PhysicsReplicationCVars
{
	namespace ResimulationCVars
	{
		int32 RedundantInputs = 2;
		static FAutoConsoleVariableRef CVarResimRedundantInputs(TEXT("np2.Resim.RedundantInputs"), RedundantInputs, TEXT("How many extra inputs to send with each unreliable network message, to account for packetloss. From owning client to server and back to owning client. NOTE: This is disabled while np2.Resim.DynamicInputReplicationScaling.Enabled is enabled. Clamped by NetworkPhysicsComponentConstants::MaxNumberOfElementsToNetwork."));
		int32 RedundantRemoteInputs = 1;
		static FAutoConsoleVariableRef CVarResimRedundantRemoteInputs(TEXT("np2.Resim.RedundantRemoteInputs"), RedundantRemoteInputs, TEXT("How many extra inputs to send with each unreliable network message, to account for packetloss. From server to remote clients. Clamped by NetworkPhysicsComponentConstants::MaxNumberOfElementsToNetwork."));
		int32 RedundantStates = 0;
		static FAutoConsoleVariableRef CVarResimRedundantStates(TEXT("np2.Resim.RedundantStates"), RedundantStates, TEXT("How many extra states to send with each unreliable network message, to account for packetloss. Clamped by NetworkPhysicsComponentConstants::MaxNumberOfElementsToNetwork."));

		bool bDynamicInputReplicationScalingEnabled = true;
		static FAutoConsoleVariableRef CVarDynamicInputReplicationScalingEnabled(TEXT("np2.Resim.DynamicInputReplicationScaling.Enabled"), bDynamicInputReplicationScalingEnabled, TEXT("Enable dynmic scaling of number of inputs sent from owning client to the server to account for packet loss. The server will control the value based on how often the server has a hole in its input buffer. NOTE: This overrides np2.Resim.RedundantInputs. Clamped by NetworkPhysicsComponentConstants::MaxNumberOfElementsToNetwork."));
		float DynamicInputReplicationScalingMaxInputsPercent = 0.1f;
		static FAutoConsoleVariableRef CVarDynamicInputReplicationScalingMaxInputsPercent(TEXT("np2.Resim.DynamicInputReplicationScaling.MaxInputsPercent"), DynamicInputReplicationScalingMaxInputsPercent, TEXT("Default 0.1 (= 10%, value in percent as multiplier). Sets the max scalable number of inputs to network from owning client to server as a percentage of the physics fixed tick-rate. 10% of 30Hz = 3 inputs at max. Clamped by NetworkPhysicsComponentConstants::MaxNumberOfElementsToNetwork."));
		int32 DynamicInputReplicationScalingMinInputs = 2;
		static FAutoConsoleVariableRef CVarDynamicInputReplicationScalingMinInputs(TEXT("np2.Resim.DynamicInputReplicationScaling.MinInputs"), DynamicInputReplicationScalingMinInputs, TEXT("Default 2. Sets the minimum scalable number of inputs to network from owning client to server."));
		float DynamicInputReplicationScalingIncreaseAverageMultiplier = 0.2f;
		static FAutoConsoleVariableRef CVarDynamicInputReplicationScalingIncreaseAverageMultiplier(TEXT("np2.Resim.DynamicInputReplicationScaling.IncreaseAverageMultiplier"), DynamicInputReplicationScalingIncreaseAverageMultiplier, TEXT("Default 0.2 (= 20%). Multiplier for how fast the average input scaling value increases. NOTE it's recommended to have a higher value than np2.Resim.DynamicInputReplicationScaling.DecreaseAverageMultiplier so the average can grow quick when network conditions gets worse."));
		float DynamicInputReplicationScalingDecreaseAverageMultiplier = 0.1f;
		static FAutoConsoleVariableRef CVarDynamicInputReplicationScalingDecreaseAverageMultiplier(TEXT("np2.Resim.DynamicInputReplicationScaling.DecreaseAverageMultiplier"), DynamicInputReplicationScalingDecreaseAverageMultiplier, TEXT("Default 0.1 (= 10%). Multiplier for how fast the average input scaling value decreases. NOTE it's recommended to have a lower value than np2.Resim.DynamicInputReplicationScaling.IncreaseAverageMultiplier so the average doesn't try to decrease too quickly which can cause repeated desyncs."));
		float DynamicInputReplicationScalingIncreaseTimeInterval = 2.0f;
		static FAutoConsoleVariableRef CVarDynamicInputReplicationScalingIncreaseTimeInterval(TEXT("np2.Resim.DynamicInputReplicationScaling.IncreaseTimeInterval"), DynamicInputReplicationScalingIncreaseTimeInterval, TEXT("Default 2.0 (value in seconds). How often dynamic scaling can increase the number of inputs to send." ));
		float DynamicInputReplicationScalingDecreaseTimeInterval = 10.0f;
		static FAutoConsoleVariableRef CVarDynamicInputReplicationScalingDecreaseTimeInterval(TEXT("np2.Resim.DynamicInputReplicationScaling.DecreaseTimeInterval"), DynamicInputReplicationScalingDecreaseTimeInterval, TEXT("Default 10.0 (value in seconds). How often dynamic scaling can decrease the number of inputs to send."));

		bool bDynamicInputBufferScalingEnabled = true;
		static FAutoConsoleVariableRef CVarDynamicInputBufferScalingEnabled(TEXT("np2.Resim.DynamicInputBufferScaling.Enabled"), bDynamicInputBufferScalingEnabled, TEXT("Enable dynmic scaling of input buffer on the server."));
		float DynamicInputBufferScalingMinBufferMs = 30.0f;
		static FAutoConsoleVariableRef CVarDynamicInputBufferScalingMinBufferMs(TEXT("np2.Resim.DynamicInputBufferScaling.MinBufferTime"), DynamicInputBufferScalingMinBufferMs, TEXT("Time in milliseconds for the lowest allowed input buffer size, when below this the buffer will scale up over time."));
		float DynamicInputBufferScalingMaxBufferMs = 90.0f;
		static FAutoConsoleVariableRef CVarDynamicInputBufferScalingMaxBufferMs(TEXT("np2.Resim.DynamicInputBufferScaling.MaxBufferTime"), DynamicInputBufferScalingMaxBufferMs, TEXT("Time in milliseconds to cap out the input buffer size."));
		float DynamicInputBufferScalingAverageTime = 0.5f;
		static FAutoConsoleVariableRef CVarDynamicInputBufferScalingAverageTime(TEXT("np2.Resim.DynamicInputBufferScaling.AverageTime"), DynamicInputBufferScalingAverageTime, TEXT("Time in seconds to keep a running average of the input buffer size over."));
		float DynamicInputBufferScalingBumpUpMultiplier = 1.0f;
		static FAutoConsoleVariableRef CVarDynamicInputBufferScalingBumpUpMultiplier(TEXT("np2.Resim.DynamicInputBufferScaling.BumpUpMultiplier"), DynamicInputBufferScalingBumpUpMultiplier, TEXT("Multiplier for how much of a fixed delta time should be added to the target input buffer size instantly."));
		float DynamicInputBufferScalingScaleUpMultiplier = 1.0f;
		static FAutoConsoleVariableRef CVarDynamicInputBufferScalingScaleUpMultiplier(TEXT("np2.Resim.DynamicInputBufferScaling.ScaleUpMultiplier"), DynamicInputBufferScalingScaleUpMultiplier, TEXT("How fast the input buffer scales up when too low. Default 1.0 = one fixed delta time per second."));
		float DynamicInputBufferScalingScaleDownMinMultiplier = 0.01f;
		static FAutoConsoleVariableRef CVarDynamicInputBufferScalingScaleDownMinMultiplier(TEXT("np2.Resim.DynamicInputBufferScaling.ScaleDownMinMultiplier"), DynamicInputBufferScalingScaleDownMinMultiplier, TEXT("How fast the input buffer scales down when buffer is slightly too large."));
		float DynamicInputBufferScalingScaleDownMaxMultiplier = 0.1f;
		static FAutoConsoleVariableRef CVarDynamicInputBufferScalingScaleDownMaxMultiplier(TEXT("np2.Resim.DynamicInputBufferScaling.ScaleDownMaxMultiplier"), DynamicInputBufferScalingScaleDownMaxMultiplier, TEXT("How fast the input buffer scales down when buffer is slightly too large."));
		bool bDynamicInputBufferScalingDebugLogs = false;
		static FAutoConsoleVariableRef CVarDynamicInputBufferScalingDebugLogs(TEXT("np2.Resim.DynamicInputBufferScaling.DebugLogs"), bDynamicInputBufferScalingDebugLogs, TEXT("Print logs for debugging."));

		bool bAllowRewindToClosestState = true;
		static FAutoConsoleVariableRef CVarResimAllowRewindToClosestState(TEXT("np2.Resim.AllowRewindToClosestState"), bAllowRewindToClosestState, TEXT("When rewinding to a specific frame, if the client doens't have state data for that frame, use closest data available. Only affects the first rewind frame, when FPBDRigidsEvolution is set to Reset."));
		bool bCompareStateToTriggerRewind = false;
		static FAutoConsoleVariableRef CVarResimCompareStateToTriggerRewind(TEXT("np2.Resim.CompareStateToTriggerRewind"), bCompareStateToTriggerRewind, TEXT("When true, cache local FNetworkPhysicsData state in rewind history and compare the predicted state with incoming server state to trigger resimulations if they differ, comparison done through FNetworkPhysicsData::CompareData. Only applies if IsLocallyControlled, to enable this for simulated proxies, where IsLocallyControlled is false, also enable np2.Resim.CompareStateToTriggerRewind.IncludeSimProxies)"));
		bool bCompareStateToTriggerRewindIncludeSimProxies = false;
		static FAutoConsoleVariableRef CVarResimCompareStateToTriggerRewindIncludeSimProxies(TEXT("np2.Resim.CompareStateToTriggerRewind.IncludeSimProxies"), bCompareStateToTriggerRewindIncludeSimProxies, TEXT("When true, include simulated proxies when np2.Resim.CompareStateToTriggerRewind is enabled."));
		bool bCompareInputToTriggerRewind = false;
		static FAutoConsoleVariableRef CVarResimCompareInputToTriggerRewind(TEXT("np2.Resim.CompareInputToTriggerRewind"), bCompareInputToTriggerRewind, TEXT("When true, compare local predicted FNetworkPhysicsData input with incoming server inputs to trigger resimulations if they differ, comparison done through FNetworkPhysicsData::CompareData."));
		bool bEnableUnreliableFlow = true;
		static FAutoConsoleVariableRef CVarResimEnableUnreliableFlow(TEXT("np2.Resim.EnableUnreliableFlow"), bEnableUnreliableFlow, TEXT("When true, allow data to be sent unreliably. Also sends FNetworkPhysicsData not marked with FNetworkPhysicsData::bimportant unreliably over the network."));
		bool bEnableReliableFlow = false;
		static FAutoConsoleVariableRef CVarResimEnableReliableFlow(TEXT("np2.Resim.EnableReliableFlow"), bEnableReliableFlow, TEXT("EXPERIMENTAL -- When true, allow data to be sent reliably. Also send FNetworkPhysicsData marked with FNetworkPhysicsData::bimportant reliably over the network."));
		bool bApplyDataInsteadOfMergeData = false;
		static FAutoConsoleVariableRef CVarResimApplyDataInsteadOfMergeData(TEXT("np2.Resim.ApplyDataInsteadOfMergeData"), bApplyDataInsteadOfMergeData, TEXT("When true, call ApplyData for each data instead of MergeData when having to use multiple data entries in one frame."));
		bool bAllowInputExtrapolation = true;
		static FAutoConsoleVariableRef CVarResimAllowInputExtrapolation(TEXT("np2.Resim.AllowInputExtrapolation"), bAllowInputExtrapolation, TEXT("When true, allow inputs to be extrapolated from last known on the server and if there is a gap allow interpolation between two known inputs."));
		bool bValidateDataOnGameThread = false;
		static FAutoConsoleVariableRef CVarResimValidateDataOnGameThread(TEXT("np2.Resim.ValidateDataOnGameThread"), bValidateDataOnGameThread, TEXT("When true, perform server-side input validation through FNetworkPhysicsData::ValidateData on the Game Thread, note that LocalFrame will be the same as ServerFrame on Game Thread. If false, perform the call on the Physics Thread."));
		bool bApplySimProxyStateAtRuntime = false;
		static FAutoConsoleVariableRef CVarResimApplySimProxyStateAtRuntime(TEXT("np2.Resim.ApplySimProxyStateAtRuntime"), bApplySimProxyStateAtRuntime, TEXT("When true, call ApplyData on received states for simulated proxies at runtime."));
		bool bApplySimProxyInputAtRuntime = true;
		static FAutoConsoleVariableRef CVarResimApplySimProxyInputAtRuntime(TEXT("np2.Resim.ApplySimProxyInputAtRuntime"), bApplySimProxyInputAtRuntime, TEXT("When true, call ApplyData on received inputs for simulated proxies at runtime."));
		bool bTriggerResimOnInputReceive = false;
		static FAutoConsoleVariableRef CVarTriggerResimOnInputReceive(TEXT("np2.Resim.TriggerResimOnInputReceive"), bTriggerResimOnInputReceive, TEXT("When true, a resim will be requested to the frame of the latest frame of received inputs this frame"));

		bool bEnableInputDecay = true;
		static FAutoConsoleVariableRef CVarEnableInputDecay(TEXT("np2.Resim.EnableInputDecay"), bEnableInputDecay, TEXT("When true, apply the Input Decay on predicted inputs."));
		bool bApplyInputDecayOverSetTime = false;
		static FAutoConsoleVariableRef CVarApplyInputDecayOverSetTime(TEXT("np2.Resim.ApplyInputDecayOverSetTime"), bApplyInputDecayOverSetTime, TEXT("When true, apply the Input Decay Curve over a set amount of time instead of over the start of input prediction and end of resim which is variable each resimulation"));
		float InputDecaySetTime = 0.15f;
		static FAutoConsoleVariableRef CVarInputDecaySetTime(TEXT("np2.Resim.InputDecaySetTime"), InputDecaySetTime, TEXT("Applied when np2.Resim.ApplyInputDecayOverSetTime is true, read there for more info. Set time to apply Input Decay Curve over while predicting inputs during resimulation"));
		bool bEnableLagScalingInputDecay = false;
		static FAutoConsoleVariableRef CVarEnableLagScaledInputDecay(TEXT("np2.Resim.EnableLagScalingInputDecay"), bEnableLagScalingInputDecay, TEXT("If true, scales input decay as a proportion of measured input lag compared to the reference input lag (np2.Resim.InputDecayReferenceLagMs)"));
		float InputDecayReferenceLagMs = 100.0f;
		static FAutoConsoleVariableRef CVarInputDecayReferenceLagMs(TEXT("np2.Resim.InputDecayReferenceLagMs"), InputDecayReferenceLagMs, TEXT("The reference input lag in milliseconds used to scale input decay. Only applies if np2.Resim.EnableLagScalingInputDecay is true"));
		bool bApplyInputDecaySimProxyInputAtRuntime = false;
		static FAutoConsoleVariableRef CVarApplyInputDecaySimProxyInputAtRuntime(TEXT("np2.Resim.ApplyInputDecaySimProxyInputAtRuntime"), bApplyInputDecaySimProxyInputAtRuntime, TEXT("When true, apply input decay on inputs applied on simulated proxies at runtime (outside of resimulation)"));
		float InputDecaySimProxyInputAtRuntime = 0.25f;
		static FAutoConsoleVariableRef CVarInputDecaySimProxyInputAtRuntime(TEXT("np2.Resim.InputDecaySimProxyInputAtRuntime"), InputDecaySimProxyInputAtRuntime, TEXT("The amount of input decay to apply on simulated proxy inputs at runtime (outside of resim), if np2.Resim.ApplyInputDecaySimProxyInputAtRuntime is true"));
		bool bEnableLagScalingSimProxyRuntimeInputDecay = false;
		static FAutoConsoleVariableRef CVarEnableLagScalingSimProxyRuntimeInputDecay(TEXT("np2.Resim.EnableLagScalingSimProxyRuntimeInputDecay"), bEnableLagScalingSimProxyRuntimeInputDecay, TEXT("When true, scale 'runtime' simulated proxy input decay with measured lag, just like resim input decay"));

		bool bActionsEnableDebugLogs = false;
		static FAutoConsoleVariableRef CVarNetworkedActionsEnableDebugLogs(TEXT("np2.Resim.NetworkedActions.EnableDebugLogs"), bActionsEnableDebugLogs, TEXT("Enable logs for the networked Actions flow in non-shipping builds."));
		int32 bActionsEquivalenceFrameWindow = 1;
		static FAutoConsoleVariableRef CVarNetworkedActionsEquivalenceFrameWindow(TEXT("np2.Resim.NetworkedActions.EquivalenceFrameWindow"), bActionsEquivalenceFrameWindow, TEXT("Number of frames to look forward and behind when looking for equivalent actions in proposals when server produces an action or client re-produces an acton during resim, as well as in predicted actions when receiving a confirmed action from the server. A value of N means look at (CurrentFrame - N), CurrentFrame and (CurrentFrame + N)."));

		bool bSimDecayNetPhysicsCompEnable = false;
		static FAutoConsoleVariableRef CVarSimDecayNetPhysicsCompEnable(TEXT("np2.Resim.SimulationDecay.NetPhysicsComp.Enable"), bSimDecayNetPhysicsCompEnable, TEXT("Enable SimulationDecay for for sim-proxies running the NetworkPhysicsComponent."));
		bool bSimDecayNetPhysicsCompApplyAtRuntime = false;
		static FAutoConsoleVariableRef CVarSimDecayNetPhysicsCompApplyAtRuntime(TEXT("np2.Resim.SimulationDecay.NetPhysicsComp.ApplyAtRuntime"), bSimDecayNetPhysicsCompApplyAtRuntime, TEXT("Apply SimulationDecay during regular (non-resim) frames in addition to resim frames, for sim-proxies running this component. Only active when the particle is in EPhysicsReplicationMode::Resimulation."));
		bool bSimDecayNetPhysicsCompUseDynamic = true;
		static FAutoConsoleVariableRef CVarSimDecayNetPhysicsCompUseDynamic(TEXT("np2.Resim.SimulationDecay.NetPhysicsComp.UseDynamic"), bSimDecayNetPhysicsCompUseDynamic, TEXT("When true, compute the clamp from the NetworkPhysicsComponent's running-average input-prediction depth. When false, use StaticTimeScale."));
		float SimDecayNetPhysicsCompStaticTimeScale = 0.9f;
		static FAutoConsoleVariableRef CVarSimDecayNetPhysicsCompStaticTimeScale(TEXT("np2.Resim.SimulationDecay.NetPhysicsComp.StaticTimeScale"), SimDecayNetPhysicsCompStaticTimeScale, TEXT("Static clamp value used when UseDynamic is false."));
		float SimDecayNetPhysicsCompDynamicBase = 0.1f;
		static FAutoConsoleVariableRef CVarSimDecayNetPhysicsCompDynamicBase(TEXT("np2.Resim.SimulationDecay.NetPhysicsComp.DynamicBase"), SimDecayNetPhysicsCompDynamicBase, TEXT("Base value added to the dynamic clamp formula."));
		float SimDecayNetPhysicsCompDynamicMin = 0.25f;
		static FAutoConsoleVariableRef CVarSimDecayNetPhysicsCompDynamicMin(TEXT("np2.Resim.SimulationDecay.NetPhysicsComp.DynamicMin"), SimDecayNetPhysicsCompDynamicMin, TEXT("Minimum clamp value allowed when the dynamic formula is in use, clamped 0-1."));
		float SimDecayNetPhysicsCompDynamicMax = 1.0f;
		static FAutoConsoleVariableRef CVarSimDecayNetPhysicsCompDynamicMax(TEXT("np2.Resim.SimulationDecay.NetPhysicsComp.DynamicMax"), SimDecayNetPhysicsCompDynamicMax, TEXT("Maximum clamp value (per-NetworkPhysicsComponent ceiling), clamped 0-1."));
		float SimDecayNetPhysicsCompAverageOverTime = 2.0f;
		static FAutoConsoleVariableRef CVarSimDecayNetPhysicsCompAverageOverTime(TEXT("np2.Resim.SimulationDecay.NetPhysicsComp.AverageOverTime"), SimDecayNetPhysicsCompAverageOverTime, TEXT("Time in seconds the NetworkPhysicsComponent running-average of input-prediction depth smooths over - the signal that drives the dynamic clamp formula."));

		bool bEnableStatefulDeltaSerialization = true;
		static FAutoConsoleVariableRef CVarResimEnableStatefulDeltaSerialization(TEXT("np2.Resim.StatefulDeltaSerialization.Enable"), bEnableStatefulDeltaSerialization, TEXT("Enables stateful delta serialization for FNetworkPhysicsData derived inputs and states. During FNetworkPhysicsData::NetSerialize there will be a valid pointer to a previous FNetworkPhysicsData which can be used for delta serialization, FNetworkPhysicsData::DeltaSourceData. NOTE: Switching this during gameplay might cause disconnections."));
		bool bUseDefaultDeltaForDeltaSourceReplication = true;
		static FAutoConsoleVariableRef CVarResimUseDefaultForDeltaSourceReplication(TEXT("np2.Resim.StatefulDeltaSerialization.UseDefaultForDeltaSourceReplication"), bUseDefaultDeltaForDeltaSourceReplication, TEXT("When false delta sources will use standard serialization when being replicated. When true there will be a valid delta source pointer to default data which can be used for delta serialization when replicating delta sources."));
		float TimeToSyncStatefulDeltaSource = 5.0f;
		static FAutoConsoleVariableRef CVarResimTimeToSyncStatefulDeltaSource(TEXT("np2.Resim.StatefulDeltaSerialization.TimeToSyncStatefulDeltaSource"), TimeToSyncStatefulDeltaSource, TEXT("The time in seconds between synchronizing the stateful delta source from server to clients."));
		
		bool bApplyPredictiveInterpolationWhenBehindServer = true;
		static FAutoConsoleVariableRef CVarResimApplyPredictiveInterpolationWhenBehindServer(TEXT("np2.Resim.ApplyPredictiveInterpolationWhenBehindServer"), bApplyPredictiveInterpolationWhenBehindServer, TEXT("When true, switch over to replicating with Predictive Interpolation temporarily, when the client receive target states from the server for frames that have not yet simulated on the client. When false apply the received target via a resimulation when the client has caught up and simulated the corresponding frame."));

		bool bRecordStatePostSolve = true;
		static FAutoConsoleVariableRef CVarResimRecordStatePostSolve(TEXT("np2.Resim.RecordStatePostSolve"), bRecordStatePostSolve, TEXT("When true, cache custom state in PostSolve_Internal (and mark it for current frame + 1) instead of between ProcessInputs_Internal and OnPreSimulate_Internal. This makes clients receive states 1 frame earlier, reducing number of frames needed to resimulate."));

		int32 bDebugTriggerResimEveryNFrames = 0;
		static FAutoConsoleVariableRef CVarDebugTriggerResimEveryNFrames(TEXT("np2.Resim.Debug.TriggerResimEveryNFrames"), bDebugTriggerResimEveryNFrames, TEXT("When above 0, trigger a resim at an interval of the set value, as long as we receive a state or input on that frame."));
	}
}

namespace Chaos
{
	extern CHAOS_API int32 RewindBeforeAdvance;
}

bool FNetworkPhysicsRewindDataProxy::NetSerializeBase(FArchive& Ar, class UPackageMap* Map, bool& bOutSuccess
	, TUniqueFunction<TUniquePtr<Chaos::FBaseRewindHistory> ()> CreateHistoryFunction
	, TUniqueFunction<const uint32 ()> GetLatestDeltaSourceIndex
	, TUniqueFunction<FNetworkPhysicsData* (const int32)> GetDeltaSourceData)
{
	bDeltaSerializationIssue = false;
	bool bHasData = false;

	if (Ar.IsSaving())
	{
		if (History.IsValid())
		{
			bHasData = History->GetHistorySize() > 0;
		}

		Ar.SerializeBits(&bHasData, 1);

		if (bHasData)
		{
			History->NetSerialize(Ar, Map, [&](void* Data, const int32 DataIndex)
				{
					if (FNetworkPhysicsData* NetData = static_cast<FNetworkPhysicsData*>(Data))
					{
						if (Owner)
						{
							// Set the component pointer to the implementation that uses this data
							NetData->SetImplementationComponent(Owner.Get()->ActorComponent.Get());

							NetData->bIsUsingDeltaSerialization = false;
							// Only use stateful delta source for the first entry in history, the following entries will use the previous entry as delta source
							if (PhysicsReplicationCVars::ResimulationCVars::bEnableStatefulDeltaSerialization && DataIndex == 0 && GetLatestDeltaSourceIndex && GetDeltaSourceData)
							{
								// Stateful Delta Serialization
								{
									uint32 DeltaSourceIndex = GetLatestDeltaSourceIndex();
									if (FNetworkPhysicsData* DeltaSourceData = GetDeltaSourceData(DeltaSourceIndex))
									{
										NetData->SetDeltaSourceData(DeltaSourceData);
									}
									else
									{
										ensureMsgf(false, TEXT("Delta Serialization failed to get the latest delta source when sending, should not happen. On the first send the latest index should be populated with a default value, not null."));
										NetData->SetDeltaSourceData(GetDeltaSourceData(/*Default*/ -2));

										// Set "index" to the buffer size, meaning it's invalid
										DeltaSourceIndex = NetworkPhysicsComponentConstants::DeltaSourceBufferSize;
									}

									Ar.SerializeBits(&NetData->bIsUsingDeltaSerialization, 1);

									if (NetData->bIsUsingDeltaSerialization)
									{
										constexpr uint32 NumBitsDeltaBufferSize = FMath::CeilLogTwo(NetworkPhysicsComponentConstants::DeltaSourceBufferSize);
										Ar.SerializeBits(&DeltaSourceIndex, NumBitsDeltaBufferSize);
									}
								}
							}
							else
							{
								Ar.SerializeBits(&NetData->bIsUsingDeltaSerialization, 1);
							}
						}
					}
				});
		}
	}
	else // IsLoading
	{
		if (!History.IsValid())
		{
			if (ensureMsgf(Owner, TEXT("FNetRewindDataBase::NetSerialize: owner is null")))
			{
				History = CreateHistoryFunction();
				if (!ensureMsgf(History.IsValid(), TEXT("FNetRewindDataBase::NetSerialize: failed to create history. Owner: %s"), *GetFullNameSafe(Owner)))
				{
					Ar.SetError();
					bOutSuccess = false;
					return true;
				}
			}
			else
			{
				Ar.SetError();
				bOutSuccess = false;
				return true;
			}
		}

		Ar.SerializeBits(&bHasData, 1);

		if (bHasData)
		{
			History->NetSerialize(Ar, Map, [&](void* Data, const int32 DataIndex)
				{
					if (FNetworkPhysicsData* NetData = static_cast<FNetworkPhysicsData*>(Data))
					{
						if (Owner)
						{
							// Set the component pointer to the implementation that uses this data
							NetData->SetImplementationComponent(Owner.Get()->ActorComponent.Get());

							Ar.SerializeBits(&NetData->bIsUsingDeltaSerialization, 1);

							// Stateful Delta Serialization
							if (NetData->bIsUsingDeltaSerialization)
							{
								uint32 DeltaSourceIndex = 0;
								constexpr uint32 NumBitsDeltaBufferSize = FMath::CeilLogTwo(NetworkPhysicsComponentConstants::DeltaSourceBufferSize);
								Ar.SerializeBits(&DeltaSourceIndex, NumBitsDeltaBufferSize); 

								// Only use stateful delta source for the first entry in history, the following entries will use the previous entry as delta source
								if (PhysicsReplicationCVars::ResimulationCVars::bEnableStatefulDeltaSerialization && GetDeltaSourceData)
								{

									FNetworkPhysicsData* DeltaSourceData = nullptr;
									if (DeltaSourceIndex < NetworkPhysicsComponentConstants::DeltaSourceBufferSize)
									{
										DeltaSourceData = GetDeltaSourceData(static_cast<int32>(DeltaSourceIndex));;
									}
									else
									{
										// Sender used default as delta source
										DeltaSourceData = GetDeltaSourceData(/*Default*/ -2);
									}

									if (!DeltaSourceData)
									{
#if DEBUG_NETWORK_PHYSICS_DELTASERIALIZATION
										UE_LOGF(LogChaos, Warning, "[DEBUG Delta Serialization] %ls ISSUE, did not find delta source at index: %d  --  Name: %ls"
											, (Owner->HasServerWorld() ? TEXT("[SERVER]    ") : (Owner->IsLocallyControlled() ? TEXT("[AUTONOMOUS]") : TEXT("[SIMULATED] "))), DeltaSourceIndex, *AActor::GetDebugName(Owner->GetOwner()));
#endif
										bDeltaSerializationIssue = true;
										DeltaSourceData = GetDeltaSourceData(/*Default*/ -2);
									}

									// Don't use the SetDeltaSourceData API since it also override bIsUsingDeltaSerialization depending on if delta source is null or not, but here we know delta was used even if we can't find a valid delta source.
									NetData->DeltaSourceData = DeltaSourceData;
								}
							}
						}
					}
				});
		}
	}

	return true;
}

FNetworkPhysicsRewindDataProxy& FNetworkPhysicsRewindDataProxy::operator=(const FNetworkPhysicsRewindDataProxy& Other)
{
	if (&Other != this)
	{
		Owner = Other.Owner;
		History = Other.History ? Other.History->Clone() : nullptr;
	}

	return *this;
}

FNetworkPhysicsRewindDataProxyRPC& FNetworkPhysicsRewindDataProxyRPC::operator=(const FNetworkPhysicsRewindDataProxyRPC& Other)
{
	if (&Other != this)
	{
		Owner = Other.Owner;
		History = Other.History ? Other.History->Clone() : nullptr;
	}

	return *this;
}

// Replicated Properties, register ReplicationFragment to inject Owner (without replicating it) into FNetworkPhysicsRewindDataProxy 
UE_NET_IMPLEMENT_NAMED_STRUCT_LASTRESORT_NETSERIALIZER_AND_REGISTRY_DELEGATES(NetworkPhysicsRewindDataOwnerInputProxy, { .CreateAndRegisterReplicationFragmentFunction = UE::Net::CreateAndRegisterNetworkPhysicsRewindDataProxyReplicationFragment })
UE_NET_IMPLEMENT_NAMED_STRUCT_LASTRESORT_NETSERIALIZER_AND_REGISTRY_DELEGATES(NetworkPhysicsRewindDataRemoteInputProxy, { .CreateAndRegisterReplicationFragmentFunction = UE::Net::CreateAndRegisterNetworkPhysicsRewindDataProxyReplicationFragment })
UE_NET_IMPLEMENT_NAMED_STRUCT_LASTRESORT_NETSERIALIZER_AND_REGISTRY_DELEGATES(NetworkPhysicsRewindDataStateProxy, { .CreateAndRegisterReplicationFragmentFunction = UE::Net::CreateAndRegisterNetworkPhysicsRewindDataProxyReplicationFragment })
UE_NET_IMPLEMENT_NAMED_STRUCT_LASTRESORT_NETSERIALIZER_AND_REGISTRY_DELEGATES(NetworkPhysicsRewindDataDeltaSourceInputProxy, { .CreateAndRegisterReplicationFragmentFunction = UE::Net::CreateAndRegisterNetworkPhysicsRewindDataProxyReplicationFragment });
UE_NET_IMPLEMENT_NAMED_STRUCT_LASTRESORT_NETSERIALIZER_AND_REGISTRY_DELEGATES(NetworkPhysicsRewindDataDeltaSourceStateProxy, { .CreateAndRegisterReplicationFragmentFunction = UE::Net::CreateAndRegisterNetworkPhysicsRewindDataProxyReplicationFragment });

// Replicated RPC Parameters, doesn't support ReplicationFragment so they replicate Owner via FNetworkPhysicsRewindDataProxyRPC
UE_NET_IMPLEMENT_NAMED_STRUCT_LASTRESORT_NETSERIALIZER_AND_REGISTRY_DELEGATES(NetworkPhysicsRewindDataInputProxy);
UE_NET_IMPLEMENT_NAMED_STRUCT_LASTRESORT_NETSERIALIZER_AND_REGISTRY_DELEGATES(NetworkPhysicsRewindDataImportantInputProxy);
UE_NET_IMPLEMENT_NAMED_STRUCT_LASTRESORT_NETSERIALIZER_AND_REGISTRY_DELEGATES(NetworkPhysicsRewindDataImportantStateProxy);

bool FNetworkPhysicsRewindDataInputProxy::NetSerialize(FArchive& Ar, class UPackageMap* Map, bool& bOutSuccess)
{
	const bool bSuccess = NetSerializeBase(Ar, Map, bOutSuccess
		/*CreateHistoryFunction*/, [this]() { return Owner->InputHelper->CreateUniqueRewindHistory(0); }
		/*GetLatestDeltaSourceIndex*/, [this]() -> const uint32 { return Owner->GetLatestAcknowledgedDeltaSourceInputIndex(); }
		/*GetDeltaSourceData*/, [this](const int32 Value) -> FNetworkPhysicsData* { return Owner->GetDeltaSourceInput(Value, /*bValueIsIndex*/ true); });

#if DEBUG_NETWORK_PHYSICS_DELTASERIALIZATION
	if (bDeltaSerializationIssue)
	{
		UE_LOGF(LogChaos, Warning, "		[DEBUG Delta Serialization] INPUT RPC");
	}
#endif
	return bSuccess;
}

bool FNetworkPhysicsRewindDataOwnerInputProxy::NetSerialize(FArchive& Ar, class UPackageMap* Map, bool& bOutSuccess)
{
	const bool bSuccess = NetSerializeBase(Ar, Map, bOutSuccess
		/*CreateHistoryFunction*/, [this]() { return Owner->InputHelper->CreateUniqueRewindHistory(0); }
		/*GetLatestDeltaSourceIndex*/, [this]() -> const uint32 { return Owner->GetLatestAcknowledgedDeltaSourceInputIndex(); }
		/*GetDeltaSourceData*/, [this](const int32 Value) -> FNetworkPhysicsData* { return Owner->GetDeltaSourceInput(Value, /*bValueIsIndex*/ true); });

#if DEBUG_NETWORK_PHYSICS_DELTASERIALIZATION
	if (bDeltaSerializationIssue)
	{
		UE_LOGF(LogChaos, Warning, "		[DEBUG Delta Serialization] OWNER INPUT");
	}
#endif
	return bSuccess;
}

bool FNetworkPhysicsRewindDataRemoteInputProxy::NetSerialize(FArchive& Ar, class UPackageMap* Map, bool& bOutSuccess)
{
	const bool bSuccess = NetSerializeBase(Ar, Map, bOutSuccess
		/*CreateHistoryFunction*/, [this]() { return Owner->InputHelper->CreateUniqueRewindHistory(0); }
		/*GetLatestDeltaSourceIndex*/, [this]() -> const uint32 { return Owner->GetLatestAcknowledgedDeltaSourceInputIndex(); }
		/*GetDeltaSourceData*/, [this](const int32 Value) -> FNetworkPhysicsData* { return Owner->GetDeltaSourceInput(Value, /*bValueIsIndex*/ true); });

#if DEBUG_NETWORK_PHYSICS_DELTASERIALIZATION
	if (bDeltaSerializationIssue) { UE_LOGF(LogChaos, Warning, "		[DEBUG Delta Serialization] REMOTE INPUT"); }
#endif

	return bSuccess;
}

bool FNetworkPhysicsRewindDataStateProxy::NetSerialize(FArchive& Ar, class UPackageMap* Map, bool& bOutSuccess)
{
	const bool bSuccess = NetSerializeBase(Ar, Map, bOutSuccess
		/*CreateHistoryFunction*/, [this]() { return Owner->StateHelper->CreateUniqueRewindHistory(0); }
		/*GetLatestDeltaSourceIndex*/, [this]() -> const uint32 { return Owner->GetLatestAcknowledgedDeltaSourceStateIndex(); }
		/*GetDeltaSourceData*/, [this](const int32 Value) -> FNetworkPhysicsData* { return Owner->GetDeltaSourceState(Value, /*bValueIsIndex*/ true); });

#if DEBUG_NETWORK_PHYSICS_DELTASERIALIZATION
	if (bDeltaSerializationIssue) { UE_LOGF(LogChaos, Warning, "		[DEBUG Delta Serialization] STATE"); }
#endif

	return bSuccess;
}

bool FNetworkPhysicsRewindDataImportantInputProxy::NetSerialize(FArchive& Ar, class UPackageMap* Map, bool& bOutSuccess)
{
	const bool bSuccess = NetSerializeBase(Ar, Map, bOutSuccess
		/*CreateHistoryFunction*/, [this]() { return Owner->InputHelper->CreateUniqueRewindHistory(0); }
		/*GetLatestDeltaSourceIndex*/, [this]() -> const uint32 { return Owner->GetLatestAcknowledgedDeltaSourceInputIndex(); }
		/*GetDeltaSourceData*/, [this](const int32 Value) -> FNetworkPhysicsData* { return Owner->GetDeltaSourceInput(Value, /*bValueIsIndex*/ true); });

#if DEBUG_NETWORK_PHYSICS_DELTASERIALIZATION
	if (bDeltaSerializationIssue) { UE_LOGF(LogChaos, Warning, "		[DEBUG Delta Serialization] IMPORTANT INPUT RPC"); }
#endif

	return bSuccess;
}

bool FNetworkPhysicsRewindDataImportantStateProxy::NetSerialize(FArchive& Ar, class UPackageMap* Map, bool& bOutSuccess)
{
	const bool bSuccess = NetSerializeBase(Ar, Map, bOutSuccess
		/*CreateHistoryFunction*/, [this]() { return Owner->StateHelper->CreateUniqueRewindHistory(0); }
		/*GetLatestDeltaSourceIndex*/, [this]() -> const uint32 { return Owner->GetLatestAcknowledgedDeltaSourceStateIndex(); }
		/*GetDeltaSourceData*/, [this](const int32 Value) -> FNetworkPhysicsData* { return Owner->GetDeltaSourceState(Value, /*bValueIsIndex*/ true); });

#if DEBUG_NETWORK_PHYSICS_DELTASERIALIZATION
	if (bDeltaSerializationIssue) { UE_LOGF(LogChaos, Warning, "		[DEBUG Delta Serialization] IMPORTANT STATE RPC"); }
#endif

	return bSuccess;
}

bool FNetworkPhysicsRewindDataDeltaSourceInputProxy::NetSerialize(FArchive& Ar, class UPackageMap* Map, bool& bOutSuccess)
{
	bool bSuccess = true;
	constexpr uint32 NumBitsDeltaBufferSize = FMath::CeilLogTwo(NetworkPhysicsComponentConstants::DeltaSourceBufferSize);
	Ar.SerializeBits(&Index, NumBitsDeltaBufferSize);

	if (PhysicsReplicationCVars::ResimulationCVars::bUseDefaultDeltaForDeltaSourceReplication)
	{
		// Use default as base for delta serialization when sending delta source
		bSuccess = NetSerializeBase(Ar, Map, bOutSuccess
			/*CreateHistoryFunction*/, [this]() { return Owner->InputHelper->CreateUniqueRewindHistory(0); }
			/*GetLatestDeltaSourceIndex*/, [this]() -> const uint32 { return Owner->GetLatestAcknowledgedDeltaSourceInputIndex(); }
			/*GetDeltaSourceData*/, [this](const int32 Value) -> FNetworkPhysicsData* { return Owner->GetDeltaSourceInput(/*Default*/ -2, /*bValueIsIndex*/ true); });
	}
	else
	{
		// Standard serialization for delta source
		bSuccess = NetSerializeBase(Ar, Map, bOutSuccess
			/*CreateHistoryFunction*/, [this]() { return Owner->InputHelper->CreateUniqueRewindHistory(0); }
			/*GetLatestDeltaSourceIndex*/, nullptr
			/*GetDeltaSourceData*/, nullptr);
	}

#if DEBUG_NETWORK_PHYSICS_DELTASERIALIZATION
	if (bDeltaSerializationIssue) { UE_LOGF(LogChaos, Warning, "		[DEBUG Delta Serialization] DELTA INPUT"); }
#endif

	return bSuccess;
}

bool FNetworkPhysicsRewindDataDeltaSourceStateProxy::NetSerialize(FArchive& Ar, class UPackageMap* Map, bool& bOutSuccess)
{
	bool bSuccess = true;
	constexpr uint32 NumBitsDeltaBufferSize = FMath::CeilLogTwo(NetworkPhysicsComponentConstants::DeltaSourceBufferSize);
	Ar.SerializeBits(&Index, NumBitsDeltaBufferSize);

	if (PhysicsReplicationCVars::ResimulationCVars::bUseDefaultDeltaForDeltaSourceReplication)
	{
		// Use default as base for delta serialization when sending delta source
		bSuccess = NetSerializeBase(Ar, Map, bOutSuccess
			/*CreateHistoryFunction*/, [this]() { return Owner->StateHelper->CreateUniqueRewindHistory(0); }
			/*GetLatestDeltaSourceIndex*/, [this]() -> const uint32 { return Owner->GetLatestAcknowledgedDeltaSourceStateIndex(); }
			/*GetDeltaSourceData*/, [this](const int32 Value) -> FNetworkPhysicsData* { return Owner->GetDeltaSourceState(/*Default*/ -2, /*bValueIsIndex*/ true); });
	}
	else
	{
		// Standard serialization for delta source
		bSuccess = NetSerializeBase(Ar, Map, bOutSuccess
			/*CreateHistoryFunction*/, [this]() { return Owner->StateHelper->CreateUniqueRewindHistory(0); }
			/*GetLatestDeltaSourceIndex*/, nullptr
			/*GetDeltaSourceData*/, nullptr);
	}

#if DEBUG_NETWORK_PHYSICS_DELTASERIALIZATION
	if (bDeltaSerializationIssue) { UE_LOGF(LogChaos, Warning, "		[DEBUG Delta Serialization] DELTA STATE"); }
#endif

	return bSuccess;
}

// --------------------------- Network Physics Callback ---------------------------

void FNetworkPhysicsCallback::InjectInputs_External(int32 PhysicsStep, int32 NumSteps)
{
	InjectInputsExternal.Broadcast(PhysicsStep, NumSteps);
}

void FNetworkPhysicsCallback::ProcessInputs_External(int32 PhysicsStep, const TArray<Chaos::FSimCallbackInputAndObject>& SimCallbackInputs)
{
	for (const Chaos::FSimCallbackInputAndObject& SimCallbackObject : SimCallbackInputs)
	{
		if (SimCallbackObject.CallbackObject
			&& SimCallbackObject.CallbackObject->HasOption(Chaos::ESimCallbackOptions::Rewind) == true
			&& SimCallbackObject.CallbackObject->HasOption(Chaos::ESimCallbackOptions::ProcessInputsExternal) == false) // Only call from here if not already listening to the native callback
		{
			SimCallbackObject.CallbackObject->ProcessInputs_External(PhysicsStep);
		}
	}
}

void FNetworkPhysicsCallback::PreProcessInputs_Internal(int32 PhysicsStep)
{
	PreProcessInputsInternal.Broadcast(PhysicsStep);
}

void FNetworkPhysicsCallback::ProcessInputs_Internal(int32 PhysicsStep, const TArray<Chaos::FSimCallbackInputAndObject>& SimCallbacks)
{
	for (Chaos::ISimCallbackObject* SimCallbackObject : RewindableCallbackObjects)
	{
		if (SimCallbackObject->HasOption(Chaos::ESimCallbackOptions::ProcessInputsInternal) == false) // Only call from here if not already listening to the native callback
		{
			SimCallbackObject->ProcessInputs_Internal(PhysicsStep);
		}
	}
}

void FNetworkPhysicsCallback::PostProcessInputs_Internal(int32 PhysicsStep)
{
	PostProcessInputsInternal.Broadcast(PhysicsStep);
}

void FNetworkPhysicsCallback::PreResimStep_Internal(int32 PhysicsStep, bool bFirst)
{
	if (bFirst)
	{
		for (Chaos::ISimCallbackObject* SimCallbackObject : RewindableCallbackObjects)
		{
			SimCallbackObject->FirstPreResimStep_Internal(PhysicsStep);
		}
	}
}

void FNetworkPhysicsCallback::PostResimStep_Internal(int32 PhysicsStep)
{

}

void FNetworkPhysicsCallback::AddResimulationRequest_Internal(const int32 PhysicsStep, const float DeltaSeconds)
{
	if (FPhysScene* PhysScene = World->GetPhysicsScene())
	{
		if (Chaos::FPhysicsSolver* PhysicsSolver = PhysScene->GetSolver())
		{
			// Add resimulation request from physics state replication
			if (IPhysicsReplicationAsync* ReplicationCallback = PhysicsSolver->GetPhysicsReplication_Internal())
			{
				ReplicationCallback->AddResimulationRequest_Internal(DeltaSeconds);
			}
		}
	}
	AddResimulationRequestInternal.Broadcast(PhysicsStep);
}

int32 FNetworkPhysicsCallback::TriggerRewindIfNeeded_Internal(int32 LatestStepCompleted)
{
	int32 ResimFrame = INDEX_NONE;
	for (Chaos::ISimCallbackObject* SimCallbackObject : RewindableCallbackObjects)
	{
		const int32 CallbackFrame = SimCallbackObject->TriggerRewindIfNeeded_Internal(LatestStepCompleted);
		ResimFrame = (ResimFrame == INDEX_NONE) ? CallbackFrame : FMath::Min(CallbackFrame, ResimFrame);
	}

	if (RewindData)
	{
		int32 TargetStateComparisonFrame = INDEX_NONE;
		if (!PhysicsReplicationCVars::ResimulationCVars::bApplyPredictiveInterpolationWhenBehindServer)
		{
			TargetStateComparisonFrame = RewindData->CompareTargetsToLastFrame();
			ResimFrame = (ResimFrame == INDEX_NONE) ? TargetStateComparisonFrame : (TargetStateComparisonFrame == INDEX_NONE) ? ResimFrame : FMath::Min(TargetStateComparisonFrame, ResimFrame);
		}

		const int32 ReplicationFrame = RewindData->GetResimFrame();
		ResimFrame = (ResimFrame == INDEX_NONE) ? ReplicationFrame : (ReplicationFrame == INDEX_NONE) ? ResimFrame : FMath::Min(ReplicationFrame, ResimFrame);

		if (ResimFrame != INDEX_NONE)
		{
			const int32 ValidFrame = RewindData->FindValidResimFrame(ResimFrame);
#if DEBUG_NETWORK_PHYSICS || DEBUG_REWIND_DATA
			UE_LOGF(LogChaos, Log, "CLIENT | PT | TriggerRewindIfNeeded_Internal | Requested Resim Frame = %d (%d / %d) | Valid Resim Frame = %d", ResimFrame, TargetStateComparisonFrame, ReplicationFrame, ValidFrame);
#endif
			ResimFrame = ValidFrame;
		}
	}
	
	return ResimFrame;
}

// --------------------------- Network Physics System ---------------------------

UNetworkPhysicsSystem::UNetworkPhysicsSystem()
{}

void UNetworkPhysicsSystem::Initialize(FSubsystemCollectionBase& Collection)
{
	UWorld* World = GetWorld();
	check(World);

	if (World->WorldType == EWorldType::PIE || World->WorldType == EWorldType::Game)
	{
		FWorldDelegates::OnPostWorldInitialization.AddUObject(this, &UNetworkPhysicsSystem::OnWorldPostInit);
	}
}

void UNetworkPhysicsSystem::Deinitialize()
{}

void UNetworkPhysicsSystem::OnWorldPostInit(UWorld* World, const UWorld::InitializationValues)
{
	if (World != GetWorld())
	{
		return;
	}

	if (UPhysicsSettings::Get()->PhysicsPrediction.bEnablePhysicsPrediction || UPhysicsSettings::Get()->PhysicsPrediction.bEnablePhysicsHistoryCapture)
	{
		if (FPhysScene* PhysScene = World->GetPhysicsScene())
		{
			if (Chaos::FPhysicsSolver* Solver = PhysScene->GetSolver())
			{ 
				if (Solver->GetRewindCallback() == nullptr)
				{
					Solver->SetRewindCallback(MakeUnique<FNetworkPhysicsCallback>(World));
				}

				if (UPhysicsSettings::Get()->PhysicsPrediction.bEnablePhysicsHistoryCapture)
				{
					if (Solver->GetRewindData() == nullptr)
					{
						Solver->EnableRewindCapture();
					}
				}
			}
		}
	}
}


// --------------------------- GameThread Network Physics Component ---------------------------

UNetworkPhysicsComponent::UNetworkPhysicsComponent(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	InitPhysics();
}

UNetworkPhysicsComponent::UNetworkPhysicsComponent() : Super()
{
	InitPhysics();
}

void UNetworkPhysicsComponent::InitPhysics()
{
	if (const IConsoleVariable* CVarRedundantInputs = IConsoleManager::Get().FindConsoleVariable(TEXT("np2.Resim.RedundantInputs")))
	{
		SetNumberOfInputsToNetwork(CVarRedundantInputs->GetInt() + 1);
	}
	if (const IConsoleVariable* CVarRedundantRemoteInputs = IConsoleManager::Get().FindConsoleVariable(TEXT("np2.Resim.RedundantRemoteInputs")))
	{
		SetNumberOfRemoteInputsToNetwork(CVarRedundantRemoteInputs->GetInt() + 1);
	}
	if (const IConsoleVariable* CVarRedundantStates = IConsoleManager::Get().FindConsoleVariable(TEXT("np2.Resim.RedundantStates")))
	{
		SetNumberOfStatesToNetwork(CVarRedundantStates->GetInt() + 1);
	}
	if (const IConsoleVariable* CVarCompareStateToTriggerRewind = IConsoleManager::Get().FindConsoleVariable(TEXT("np2.Resim.CompareStateToTriggerRewind")))
	{
		bCompareStateToTriggerRewind = CVarCompareStateToTriggerRewind->GetBool();
	}
	if (const IConsoleVariable* CVarCompareStateToTriggerRewind = IConsoleManager::Get().FindConsoleVariable(TEXT("np2.Resim.CompareStateToTriggerRewind.IncludeSimProxies")))
	{
		bCompareStateToTriggerRewindIncludeSimProxies = CVarCompareStateToTriggerRewind->GetBool();
	}
	if (const IConsoleVariable* CVarCompareInputToTriggerRewind = IConsoleManager::Get().FindConsoleVariable(TEXT("np2.Resim.CompareInputToTriggerRewind")))
	{
		bCompareInputToTriggerRewind = CVarCompareInputToTriggerRewind->GetBool();
	}

	/** NOTE:
	* If the NetworkPhysicsComponent is added as a SubObject after the actor has processed bAutoActivate
	* SetActive(true) and RegisterComponent() needs to be called manually for the component to function properly. */
	PrimaryComponentTick.bCanEverTick = true;
	PrimaryComponentTick.TickGroup = TG_PostPhysics;
	bAutoActivate = true;

	// Request InitializeComponent so the normal init path runs after all sibling components have
	// called their own OnRegister (ensuring NetworkPhysicsSettingsComponent data is available).
	// The seamless travel case is handled separately in OnRegister via bActorSeamlessTraveled.
	bWantsInitializeComponent = true;
	SetIsReplicatedByDefault(true);
}

void UNetworkPhysicsComponent::OnRegister()
{
	Super::OnRegister();

	// During seamless travel the actor skips the normal initialization path, so
	// InitializeComponent will NOT be called. We must initialize here instead.
	// For all other cases (normal spawn, level load, dynamic component addition)
	// we defer to InitializeComponent which runs after all sibling components have
	// completed their OnRegister, guaranteeing NetworkPhysicsSettingsComponent data
	// is available when we look for it.
	if (AActor* Owner = GetOwner())
	{
		if (Owner->bActorSeamlessTraveled)
		{
			InitializePhysicsReplication();
		}
	}
}

void UNetworkPhysicsComponent::InitializeComponent()
{
	Super::InitializeComponent();

	// All sibling components have called OnRegister by the time InitializeComponent runs,
	// so any NetworkPhysicsSettingsComponent data is guaranteed to be available here.
	InitializePhysicsReplication();
}

void UNetworkPhysicsComponent::InitializePhysicsReplication()
{
	// Cache CVar values
	bEnableUnreliableFlow = PhysicsReplicationCVars::ResimulationCVars::bEnableUnreliableFlow;
	bEnableReliableFlow = PhysicsReplicationCVars::ResimulationCVars::bEnableReliableFlow;
	bValidateDataOnGameThread = PhysicsReplicationCVars::ResimulationCVars::bValidateDataOnGameThread;

	if (AActor* Owner = GetOwner())
	{
		Owner->SetCallPreReplication(true);

		// Get settings from NetworkPhysicsSettingsComponent, if there is one
		UNetworkPhysicsSettingsComponent* SettingsComponent = Owner->FindComponentByClass<UNetworkPhysicsSettingsComponent>();
		if (SettingsComponent)
		{
			const FNetworkPhysicsSettingsData& SettingsData = SettingsComponent->GetSettings();
			SetNumberOfInputsToNetwork(SettingsData.NetworkPhysicsComponentSettings.GetRedundantInputs() + 1);
			SetNumberOfRemoteInputsToNetwork(SettingsData.NetworkPhysicsComponentSettings.GetRedundantRemoteInputs() + 1);
			SetNumberOfStatesToNetwork(SettingsData.NetworkPhysicsComponentSettings.GetRedundantStates() + 1);
			bEnableUnreliableFlow = SettingsData.NetworkPhysicsComponentSettings.GetEnableUnreliableFlow();
			bEnableReliableFlow = SettingsData.NetworkPhysicsComponentSettings.GetEnableReliableFlow();
			bValidateDataOnGameThread = SettingsData.NetworkPhysicsComponentSettings.GetValidateDataOnGameThread();

			if (ReplicatedOwnerInputs.History)
			{
				ReplicatedOwnerInputs.History->ResizeDataHistory(InputsToNetwork_OwnerDefault);
			}
			if (ReplicatedRemoteInputs.History)
			{
				ReplicatedRemoteInputs.History->ResizeDataHistory(InputsToNetwork_Simulated);
			}
			if (ReplicatedStates.History)
			{
				ReplicatedStates.History->ResizeDataHistory(StatesToNetwork);
			}
		}

		if (!PhysicsObject)
		{
			if (UPrimitiveComponent* RootPrimComp = Cast<UPrimitiveComponent>(Owner->GetRootComponent()))
			{
				SetPhysicsObject(RootPrimComp->GetPhysicsObjectByName(NAME_None));
			}
		}
	}

	if (UWorld* World = GetWorld())
	{
		if (FPhysScene* PhysScene = World->GetPhysicsScene())
		{
			if (Chaos::FPhysicsSolver* Solver = PhysScene->GetSolver())
			{
				// Create async component to run on Physics Thread
				NetworkPhysicsComponent_Internal = Solver->CreateAndRegisterSimCallbackObject_External<FAsyncNetworkPhysicsComponent>();
				NetworkPhysicsComponent_Internal->PhysicsObject = PhysicsObject;
				NetworkPhysicsComponent_Internal->InputsToNetwork_OwnerDefault = InputsToNetwork_OwnerDefault;
				NetworkPhysicsComponent_Internal->InputsToNetwork_Simulated = InputsToNetwork_Simulated;
				NetworkPhysicsComponent_Internal->StatesToNetwork = StatesToNetwork;
				NetworkPhysicsComponent_Internal->bCompareStateToTriggerRewind = bCompareStateToTriggerRewind;
				NetworkPhysicsComponent_Internal->bCompareStateToTriggerRewindIncludeSimProxies = bCompareStateToTriggerRewindIncludeSimProxies;
				NetworkPhysicsComponent_Internal->bCompareInputToTriggerRewind = bCompareInputToTriggerRewind;
				CreateAsyncDataHistory();
				UpdateAsyncComponent(true);

				// If a NetworkPhysicsSettingsComponent exists but its internal (physics thread) data
				// is not yet initialized, it means the settings component has not yet called its own
				// OnRegister (registration order race). The UpdateAsyncComponent call above therefore
				// could not set the physics thread SettingsComponent pointer. Flag a deferred full
				// update so it is re-sent on the first tick, by which time all sibling components
				// will have completed their OnRegister and InitializeInternalSettings will have run.
				if (AActor* SettingsOwner = GetOwner())
				{
					if (UNetworkPhysicsSettingsComponent* SettingsComp = SettingsOwner->FindComponentByClass<UNetworkPhysicsSettingsComponent>())
					{
						if (!SettingsComp->GetSettings_Internal().IsValid())
						{
							bNeedsFullAsyncComponentUpdate = true;
						}
					}
				}

				/** Run OnInitialize_Internal on the ISimCallbackObject first thing on the next physics thread frame */
				FAsyncNetworkPhysicsComponent* AsyncNetworkPhysicsComponent = NetworkPhysicsComponent_Internal;
				Solver->EnqueueCommandImmediate(
					[AsyncNetworkPhysicsComponent]()
					{
						if (AsyncNetworkPhysicsComponent)
						{
							AsyncNetworkPhysicsComponent->OnInitialize_Internal();
						}
					}
				);
			}
		}
	}
}

void UNetworkPhysicsComponent::OnUnregister()
{
	Super::OnUnregister();

	UninitializePhysicsReplication();
}

void UNetworkPhysicsComponent::UninitializePhysicsReplication()
{
	if (NetworkPhysicsComponent_Internal)
	{
		if (FAsyncNetworkPhysicsComponentInput* AsyncInput = NetworkPhysicsComponent_Internal->GetProducerInputData_External())
		{
			AsyncInput->ActorComponent = nullptr;
			AsyncInput->PhysicsObject = nullptr;
			AsyncInput->ImplementationInterface_Internal = nullptr;
			AsyncInput->ActionHandler_Internal = nullptr;
		}

		if (UWorld* World = GetWorld())
		{
			if (FPhysScene* PhysScene = World->GetPhysicsScene())
			{
				if (Chaos::FPhysicsSolver* Solver = PhysScene->GetSolver())
				{
					/* Run OnUninitialize_Internal on the ISimCallbackObject as a way to unregister input / state history, unsubscribe from delegates etc.
					* After UnregisterAndFreeSimCallbackObject_External the ISimCallbackObject will not get any callbacks anymore, use this as the last safe place to use the cached FPhysicsObject for example */
					FAsyncNetworkPhysicsComponent* AsyncNetworkPhysicsComponent = NetworkPhysicsComponent_Internal;
					Solver->EnqueueCommandImmediate(
						[AsyncNetworkPhysicsComponent]()
						{
							if (AsyncNetworkPhysicsComponent)
							{
								AsyncNetworkPhysicsComponent->OnUninitialize_Internal();
							}
						}
					);

					// Clear async component from Physics Thread and memory
					Solver->UnregisterAndFreeSimCallbackObject_External(NetworkPhysicsComponent_Internal);
				}
			}
		}
	}

	NetworkPhysicsComponent_Internal = nullptr;
	PhysicsObject = nullptr;
}

void UNetworkPhysicsComponent::BeginPlay()
{
	Super::BeginPlay();

	// Update async component with current component properties
	UpdateAsyncComponent(true);
}

#if UE_WITH_REMOTE_OBJECT_HANDLE
void UNetworkPhysicsComponent::PostMigrate(const struct FUObjectMigrationContext& MigrationContext)
{
	Super::PostMigrate(MigrationContext);

	if (MigrationContext.MigrationSide == EObjectMigrationSide::Send)
	{
		// Add extra check to avoid double call back unregistration 
		if (NetworkPhysicsComponent_Internal)
		{
			// Need to uninitialize the component during migration, because the component is not GC'ed until next frame
			// Then the call back on AsyncPhysicsNetworkComponent would cause a crash
			UninitializePhysicsReplication();
		}
	}

	if (MigrationContext.MigrationSide == EObjectMigrationSide::Receive)
	{
		// Add extra check to avoid double call back registration 
		if (!NetworkPhysicsComponent_Internal)
		{
			// Need to reinitialize the component (excluding initialization of the base)
			// The reason is ActorComponents do not get reinitialized after migrated to another server since all data on ActorComponent
			// is serialized and migrated over. 
			InitializePhysicsReplication();
		}
	}
}
#endif

void UNetworkPhysicsComponent::GetLifetimeReplicatedProps(TArray< FLifetimeProperty >& OutLifetimeProps) const
{
	Super::GetLifetimeReplicatedProps(OutLifetimeProps);

	FDoRepLifetimeParams ReplicatedParamsOwner;
	ReplicatedParamsOwner.Condition = COND_OwnerOnly;
	ReplicatedParamsOwner.RepNotifyCondition = REPNOTIFY_Always;
	ReplicatedParamsOwner.bIsPushBased = true;

	FDoRepLifetimeParams ReplicatedParamsRemote;
	ReplicatedParamsRemote.Condition = COND_SkipOwner;
	ReplicatedParamsRemote.RepNotifyCondition = REPNOTIFY_Always;
	ReplicatedParamsRemote.bIsPushBased = true;

	FDoRepLifetimeParams ReplicatedParamsAll;
	ReplicatedParamsAll.Condition = COND_Custom;
	ReplicatedParamsAll.RepNotifyCondition = REPNOTIFY_Always;
	ReplicatedParamsAll.bIsPushBased = true;

	DOREPLIFETIME_CONDITION(UNetworkPhysicsComponent, InputsToNetwork_Owner, COND_OwnerOnly);

	// RepGraph / Legacy
	DOREPLIFETIME_WITH_PARAMS_FAST(UNetworkPhysicsComponent, ReplicatedDeltaSourceInput, ReplicatedParamsAll);
	DOREPLIFETIME_WITH_PARAMS_FAST(UNetworkPhysicsComponent, ReplicatedDeltaSourceState, ReplicatedParamsAll);
	DOREPLIFETIME_WITH_PARAMS_FAST(UNetworkPhysicsComponent, ReplicatedOwnerInputs, ReplicatedParamsOwner);
	DOREPLIFETIME_WITH_PARAMS_FAST(UNetworkPhysicsComponent, ReplicatedRemoteInputs, ReplicatedParamsRemote);
	DOREPLIFETIME_WITH_PARAMS_FAST(UNetworkPhysicsComponent, ReplicatedStates, ReplicatedParamsAll);

	// Iris
	DOREPLIFETIME_WITH_PARAMS_FAST(UNetworkPhysicsComponent, ReplicatedInputCollection, ReplicatedParamsOwner);
	DOREPLIFETIME_WITH_PARAMS_FAST(UNetworkPhysicsComponent, ReplicatedRemoteInputCollection, ReplicatedParamsRemote);
	DOREPLIFETIME_WITH_PARAMS_FAST(UNetworkPhysicsComponent, ReplicatedStateCollection, ReplicatedParamsAll);
	DOREPLIFETIME_WITH_PARAMS_FAST(UNetworkPhysicsComponent, ReplicatedActionsCollection, ReplicatedParamsAll);
}

void UNetworkPhysicsComponent::PreReplication(IRepChangedPropertyTracker& ChangedPropertyTracker)
{
	Super::PreReplication(ChangedPropertyTracker);

	// Legacy - only active when input/state is registered AND using legacy
	const bool bLegacyActive = bIsUsingLegacyData && bHasRegisteredInputState;
	DOREPLIFETIME_ACTIVE_OVERRIDE_FAST(UNetworkPhysicsComponent, ReplicatedDeltaSourceInput, bLegacyActive);
	DOREPLIFETIME_ACTIVE_OVERRIDE_FAST(UNetworkPhysicsComponent, ReplicatedDeltaSourceState, bLegacyActive);
	DOREPLIFETIME_ACTIVE_OVERRIDE_FAST(UNetworkPhysicsComponent, ReplicatedOwnerInputs, bLegacyActive);
	DOREPLIFETIME_ACTIVE_OVERRIDE_FAST(UNetworkPhysicsComponent, ReplicatedRemoteInputs, bLegacyActive);
	DOREPLIFETIME_ACTIVE_OVERRIDE_FAST(UNetworkPhysicsComponent, ReplicatedStates, bLegacyActive);

	// Iris - only active when input/state is registered AND not using legacy
	const bool bIrisActive = !bIsUsingLegacyData && bHasRegisteredInputState;
	DOREPLIFETIME_ACTIVE_OVERRIDE_FAST(UNetworkPhysicsComponent, ReplicatedInputCollection, bIrisActive);
	DOREPLIFETIME_ACTIVE_OVERRIDE_FAST(UNetworkPhysicsComponent, ReplicatedRemoteInputCollection, bIrisActive);
	DOREPLIFETIME_ACTIVE_OVERRIDE_FAST(UNetworkPhysicsComponent, ReplicatedStateCollection, bIrisActive);

	// Actions - only active when there are pending actions to replicate
	DOREPLIFETIME_ACTIVE_OVERRIDE_FAST(UNetworkPhysicsComponent, ReplicatedActionsCollection, bHasPendingActionsToReplicate);
}

// Called every Game Thread frame
void UNetworkPhysicsComponent::TickComponent(float DeltaTime, ELevelTick TickType, FActorComponentTickFunction* ThisTickFunction)
{
	Super::TickComponent(DeltaTime, TickType, ThisTickFunction);

	UpdateAsyncComponent(false);

	// If InitializePhysicsReplication ran while a sibling NetworkPhysicsSettingsComponent had
	// not yet called its own OnRegister (registration order race during seamless travel), the
	// physics thread SettingsComponent pointer was not set. Send a full update now that all
	// components are guaranteed to have registered.
	if (bNeedsFullAsyncComponentUpdate)
	{
		bNeedsFullAsyncComponentUpdate = false;
		UpdateAsyncComponent(true);
	}

	NetworkMarshaledData();
}

void UNetworkPhysicsComponent::NetworkMarshaledData()
{
	if (!NetworkPhysicsComponent_Internal)
	{
		return;
	}

	UWorld* World = GetWorld();
	if (!World)
	{
		return;
	}

	// Reset pending actions flag, will be set if any actions are produced this frame
	bHasPendingActionsToReplicate = false;

	const bool bIsServer = HasServerWorld();
	if (!bIsServer && !IsNetworkPhysicsTickOffsetAssigned())
	{
		// Don't replicate data to the server until networked physics is setup with a synchronized physics tick offset
		return;
	}

	const bool bShouldSyncDeltaSourceInput = bIsServer && PhysicsReplicationCVars::ResimulationCVars::bEnableStatefulDeltaSerialization && World->GetRealTimeSeconds() > TimeToSyncDeltaSourceInput;
	const bool bShouldSyncDeltaSourceState = bIsServer && PhysicsReplicationCVars::ResimulationCVars::bEnableStatefulDeltaSerialization && World->GetRealTimeSeconds() > TimeToSyncDeltaSourceState;
	bool bHasSyncedDeltaSourceInput = false;
	bool bHasSyncedDeltaSourceState = false;

	// Replicate source data for input delta serialization
	auto DeltaSourceInputSyncHelper = [&](const TUniquePtr<Chaos::FBaseRewindHistory>& InputData)
	{
		if (bShouldSyncDeltaSourceInput && !bHasSyncedDeltaSourceInput)
		{
			ReplicatedDeltaSourceInput.History->SetRecordDataIncremental(true); // Only record data that is newer than already cached data
			if (InputData->CopyAllData(*ReplicatedDeltaSourceInput.History, /*bIncludeUnimportant*/ true, /*bIncludeImportant*/ true))
			{
				ReplicatedDeltaSourceInput.Index = GetNextDeltaSourceInputIndex();
				bHasSyncedDeltaSourceInput = true;
				TimeToSyncDeltaSourceInput = World->GetRealTimeSeconds() + PhysicsReplicationCVars::ResimulationCVars::TimeToSyncStatefulDeltaSource;
				AddDeltaSourceInput();
				MARK_PROPERTY_DIRTY_FROM_NAME(UNetworkPhysicsComponent, ReplicatedDeltaSourceInput, this);
			}
		}
	};

	// Replicate source data for state delta serialization
	auto DeltaSourceStateSyncHelper = [&](const TUniquePtr<Chaos::FBaseRewindHistory>& StateData)
	{
		if (bShouldSyncDeltaSourceState && !bHasSyncedDeltaSourceState)
		{
			ReplicatedDeltaSourceState.History->SetRecordDataIncremental(true); // Only record data that is newer than already cached data
			if (StateData->CopyAllData(*ReplicatedDeltaSourceState.History, /*bIncludeUnimportant*/ true, /*bIncludeImportant*/ true))
			{
				ReplicatedDeltaSourceState.Index = GetNextDeltaSourceStateIndex();
				bHasSyncedDeltaSourceState = true;
				TimeToSyncDeltaSourceState = World->GetRealTimeSeconds() + PhysicsReplicationCVars::ResimulationCVars::TimeToSyncStatefulDeltaSource;
				AddDeltaSourceState();
				MARK_PROPERTY_DIRTY_FROM_NAME(UNetworkPhysicsComponent, ReplicatedDeltaSourceState, this);
			}
		}
	};

	while (Chaos::TSimCallbackOutputHandle<FAsyncNetworkPhysicsComponentOutput> AsyncOutput = NetworkPhysicsComponent_Internal->PopFutureOutputData_External())
	{
		if (AsyncOutput->InputsToNetwork_Owner.IsSet())
		{
			// Only marshaled from PT to GT on the server, InputsToNetwork_Owner is a replicated property towards the owner
			InputsToNetwork_Owner = *AsyncOutput->InputsToNetwork_Owner;
		}

		if (AsyncOutput->TargetBufferSizeMs.IsSet())
		{
			// Only marshaled from PT to GT on the server
			if (APlayerController* PC = GetPlayerController())
			{
				PC->SetTickOffsetBufferTime(*AsyncOutput->TargetBufferSizeMs);
			}
		}

		// Unimportant / Unreliable
		if (bEnableUnreliableFlow
			&& AsyncOutput->InputData
			&& AsyncOutput->InputData->HasDataInHistory())
		{
			if (bIsServer)
			{
				if (bIsUsingLegacyData)
				{
					// Replicate source data for delta serialization
					DeltaSourceInputSyncHelper(AsyncOutput->InputData);
				}

				if (IsLocallyControlled())
				{
					if (bIsUsingLegacyData)
					{
						// Send inputs to remote clients after getting marshaled from PT if server is the one controlling the component
						ReplicatedRemoteInputs.History->SetRecordDataIncremental(true); // Only record data that is newer than already cached data
						if (AsyncOutput->InputData->CopyAllData(*ReplicatedRemoteInputs.History, /*bIncludeUnimportant*/ true, /*bIncludeImportant*/ true))
						{
							MARK_PROPERTY_DIRTY_FROM_NAME(UNetworkPhysicsComponent, ReplicatedRemoteInputs, this);
						}
					}
					else
					{
						// Send inputs to remote clients after getting marshaled from PT if server is the one controlling the component
						ReplicatedRemoteInputCollection.DataArray.SetNum(InputsToNetwork_Simulated, EAllowShrinking::Yes);
						InputHelper->CopyIncrementalData(AsyncOutput->InputData.Get(), ReplicatedRemoteInputCollection);
						MARK_PROPERTY_DIRTY_FROM_NAME(UNetworkPhysicsComponent, ReplicatedRemoteInputCollection, this);
					}
				}

				if (bIsUsingLegacyData)
				{
					// Only replicate data to owning client if bDataAltered is true i.e. the input has been altered by the server
					ReplicatedOwnerInputs.History->ResizeDataHistory(AsyncOutput->InputData->CountAlteredData(/*bIncludeUnimportant*/ true, /*bIncludeImportant*/ bEnableReliableFlow == false), EAllowShrinking::Yes);
					ReplicatedOwnerInputs.History->SetRecordDataIncremental(true); // Only record data that is newer than already cached data

					// Server sends inputs through property replication to owning client
					if (AsyncOutput->InputData->CopyAlteredData(*ReplicatedOwnerInputs.History, /*bIncludeUnimportant*/ true, /*bIncludeImportant*/ bEnableReliableFlow == false))
					{
						MARK_PROPERTY_DIRTY_FROM_NAME(UNetworkPhysicsComponent, ReplicatedOwnerInputs, this);
					}
				}
				else
				{
					// Server sends inputs through property replication to owning client
					ReplicatedInputCollection.DataArray.SetNum(AsyncOutput->InputData->CountAlteredData(/*bIncludeUnimportant*/ true, /*bIncludeImportant*/ bEnableReliableFlow == false), EAllowShrinking::Yes);
					if (ReplicatedInputCollection.DataArray.Num() > 0)
					{
						InputHelper->CopyAlteredData(AsyncOutput->InputData.Get(), ReplicatedInputCollection);
					}
					MARK_PROPERTY_DIRTY_FROM_NAME(UNetworkPhysicsComponent, ReplicatedInputCollection, this);
				}
			}
			else if (IsLocallyControlled()) // Client-side
			{
				if (bIsUsingLegacyData)
				{
					ReplicateClientInputs.History->ResizeDataHistory(AsyncOutput->InputData->GetHistorySize(), EAllowShrinking::Yes);
					if (AsyncOutput->InputData->CopyAllData(*ReplicateClientInputs.History, /*bIncludeUnimportant*/ true, /*bIncludeImportant*/ bEnableReliableFlow == false))
					{
						// Clients send inputs through an RPC to the server
						ServerReceiveInputData(ReplicateClientInputs);
					}
				}
				else
				{
					// Clients send inputs through an RPC to the server
					ReplicatedInputCollection.DataArray.SetNum(AsyncOutput->InputData->GetHistorySize(), EAllowShrinking::Yes);
					InputHelper->CopyData(AsyncOutput->InputData.Get(), ReplicatedInputCollection);
					ServerReceiveInputCollection(ReplicatedInputCollection);
				}
			}
		}

		// Important / Reliable
		if (bEnableReliableFlow)
		{
			for (const TUniquePtr<Chaos::FBaseRewindHistory>& InputImportant : AsyncOutput->InputDataImportant)
			{
				if (!InputImportant || !InputImportant->HasDataInHistory())
				{
					continue;
				}

				if (bIsUsingLegacyData)
				{
					// Replicate source data for delta serialization
					DeltaSourceInputSyncHelper(InputImportant);

					ReplicatedImportantInput.History->ResizeDataHistory(InputImportant->GetHistorySize(), EAllowShrinking::Yes);
					if (InputImportant->CopyAllData(*ReplicatedImportantInput.History, /*bIncludeUnimportant*/ false, /*bIncludeImportant*/ true))
					{
						if (bIsServer)
						{
							MulticastReceiveImportantInputData(ReplicatedImportantInput);
						}
						else if (IsLocallyControlled())
						{
							ServerReceiveImportantInputData(ReplicatedImportantInput);
						}
					}
				}
				else
				{
					// TODO
				}
			}
		}

		if (bIsServer)
		{
			// Unimportant / Unreliable
			if (bEnableUnreliableFlow
				&& AsyncOutput->StateData
				&& AsyncOutput->StateData->HasDataInHistory())
			{
				if (bIsUsingLegacyData)
				{
					// Replicate source data for delta serialization
					DeltaSourceStateSyncHelper(AsyncOutput->StateData);

					if (AsyncOutput->StateData->CopyAllData(*ReplicatedStates.History, /*bIncludeUnimportant*/ true, /*bIncludeImportant*/ bEnableReliableFlow == false))
					{
						// If on server we should send the states onto all the clients through repnotify
						MARK_PROPERTY_DIRTY_FROM_NAME(UNetworkPhysicsComponent, ReplicatedStates, this);
					}
				}
				else
				{
					ReplicatedStateCollection.DataArray.SetNum(StatesToNetwork, EAllowShrinking::Yes);
					StateHelper->CopyData(AsyncOutput->StateData.Get(), ReplicatedStateCollection);
					MARK_PROPERTY_DIRTY_FROM_NAME(UNetworkPhysicsComponent, ReplicatedStateCollection, this);
				}
			}

			// Important / Reliable
			if (bEnableReliableFlow)
			{
				for (const TUniquePtr<Chaos::FBaseRewindHistory>& StateImportant : AsyncOutput->StateDataImportant)
				{
					if (!StateImportant || !StateImportant->HasDataInHistory())
					{
						continue;
					}

					if (bIsUsingLegacyData)
					{
						// Replicate source data for delta serialization
						DeltaSourceStateSyncHelper(StateImportant);

						ReplicatedImportantState.History->ResizeDataHistory(StateImportant->GetHistorySize(), EAllowShrinking::Yes);
						if (StateImportant->CopyAllData(*ReplicatedImportantState.History, /*bIncludeUnimportant*/ false, /*bIncludeImportant*/ true))
						{
							MulticastReceiveImportantStateData(ReplicatedImportantState);
						}
					}
					else
					{
						// TODO
					}
				}
			}
		}

		// Unreliable Actions
		if (AsyncOutput->UnreliableActionData.Num() > 0)
		{
			bHasPendingActionsToReplicate = true;

			ReplicatedActionsCollection.DataArray = MoveTemp(AsyncOutput->UnreliableActionData);

			if (bIsServer)
			{
				MARK_PROPERTY_DIRTY_FROM_NAME(UNetworkPhysicsComponent, ReplicatedActionsCollection, this);
			}
			else
			{
				ServerReceiveUnreliableActionCollection(ReplicatedActionsCollection);
			}
		}

		// Reliable Actions (always enabled, not gated by bEnableReliableFlow)
		if (AsyncOutput->ReliableActionData.Num() > 0)
		{
			FNetworkPhysicsActionCollection ReliableActionsToSend;
			ReliableActionsToSend.DataArray = MoveTemp(AsyncOutput->ReliableActionData);

			if (bIsServer)
			{
				MulticastReceiveReliableActionCollection(ReliableActionsToSend);
			}
			else
			{
				ServerReceiveReliableActionCollection(ReliableActionsToSend);
			}
		}

		if (bStopRelayingLocalInputsDeferred)
		{
			bIsRelayingLocalInputs = false;
			bStopRelayingLocalInputsDeferred = false;
		}
	}
}

void UNetworkPhysicsComponent::OnRep_SetReplicatedDeltaSourceInput()
{
	if (ReplicatedDeltaSourceInput.bDeltaSerializationIssue)
	{
		ensureMsgf(false, TEXT("OnRep_SetReplicatedDeltaSourceInput failed delta serialization, should not happen."));
		return;
	}

	if (!ReplicatedDeltaSourceInput.History)
	{
		return;
	}
	ensure(bIsUsingLegacyData);

	AddDeltaSourceInput();
}

void UNetworkPhysicsComponent::ServerReceiveDeltaSourceInputFrame_Implementation(const uint32 Index, const int32 Frame)
{
	ensure(bIsUsingLegacyData);

	if (Index >= static_cast<uint32>(DeltaSourceInputs.Num()))
	{
#if DEBUG_NETWORK_PHYSICS_DELTASERIALIZATION
		UE_LOGF(LogChaos, Log, "[DEBUG Delta Serialization] [SERVER]     Owner failed to acknowledged delta source INPUT for frame: %d, index larger than collection: %d >= %d  --  Name: %ls", Frame, Index, DeltaSourceInputs.Num(), *AActor::GetDebugName(GetOwner()));
#endif
		return;
	}

	TUniquePtr<FNetworkPhysicsData>& Data = DeltaSourceInputs[Index];
	if (Data->ServerFrame == Frame)
	{
		// Set latest delta source index acknowledged by the client so that we can start using this delta source
		LatestAcknowledgedDeltaSourceInputIndex = Index;

#if DEBUG_NETWORK_PHYSICS_DELTASERIALIZATION
		UE_LOGF(LogChaos, Log, "[DEBUG Delta Serialization] [SERVER]     Owner acknowledged delta source INPUT frame: %d at index: %d  --  Name: %ls", Frame, LatestAcknowledgedDeltaSourceInputIndex, *AActor::GetDebugName(GetOwner()));
#endif
	}
	else
	{
#if DEBUG_NETWORK_PHYSICS_DELTASERIALIZATION
		UE_LOGF(LogChaos, Log, "[DEBUG Delta Serialization] [SERVER]     Owner failed to acknowledged delta source INPUT for Index: %d - AckFrame %d is not equal to CachedFrame: %d  --  Name: %ls", LatestAcknowledgedDeltaSourceInputIndex, Frame, Data->ServerFrame, *AActor::GetDebugName(GetOwner()));
#endif
	}

}

void UNetworkPhysicsComponent::AddDeltaSourceInput()
{
	// Get the data entry for the correct index in the data sources array
	const int32 Index = ReplicatedDeltaSourceInput.Index;
	if (Index >= DeltaSourceInputs.Num())
	{
		check(false);
		return;
	}

	FNetworkPhysicsData* PhysicsData = DeltaSourceInputs[Index].Get();

	// Extract the data from the replicated DeltaSources property
	if (ReplicatedDeltaSourceInput.History->ExtractData(ReplicatedDeltaSourceInput.History->GetLatestFrame(), /*bResetSolver*/false, PhysicsData, /*bExactFrame*/true))
	{
		// The data is now extracted via PhysicsData and stored inside DeltaSourceInputs
		
		if (!HasServerWorld())
		{
			// On the client, set the latest index, to be used when sending inputs to the server
			LatestAcknowledgedDeltaSourceInputIndex = Index;

#if DEBUG_NETWORK_PHYSICS_DELTASERIALIZATION
			UE_LOGF(LogChaos, Log, "[DEBUG Delta Serialization] %ls Received delta source INPUT for frame: %d at index: %d  --  Name: %ls", (IsLocallyControlled() ? TEXT("[AUTONOMOUS]") : TEXT("[SIMULATED] ")), ReplicatedDeltaSourceInput.History->GetLatestFrame(), LatestAcknowledgedDeltaSourceInputIndex, *AActor::GetDebugName(GetOwner()));
#endif
			if (IsLocallyControlled())
			{
				// If this client is the one controlling this entity, send back an acknowledgment to the server we have received the delta source for ServerFrame
				ServerReceiveDeltaSourceInputFrame(Index, PhysicsData->ServerFrame);
			}
		}
		else if (IsLocallyControlled())
		{
			// If server is locally controlled, set the latest index directly, else wait for the owning client to send back ServerReceiveDeltaSourceInputFrame before the server starts to use this 
			LatestAcknowledgedDeltaSourceInputIndex = Index;
		}

#if DEBUG_NETWORK_PHYSICS_DELTASERIALIZATION
		if (HasServerWorld())
		{
			UE_LOGF(LogChaos, Log, "[DEBUG Delta Serialization] [SERVER]     Sent delta source INPUT for frame: %d at index: %d  --  Name: %ls", ReplicatedDeltaSourceInput.History->GetLatestFrame(), Index, *AActor::GetDebugName(GetOwner()));
		}
#endif

		LatestCachedDeltaSourceInputIndex = Index;
	}
}

void UNetworkPhysicsComponent::OnRep_SetReplicatedDeltaSourceState()
{
	if (ReplicatedDeltaSourceState.bDeltaSerializationIssue)
	{
		ensureMsgf(false, TEXT("OnRep_SetReplicatedDeltaSourceState failed delta serialization, should not happen."));
		return;
	}

	if (!ReplicatedDeltaSourceState.History)
	{
		return;
	}
	ensure(bIsUsingLegacyData);

	AddDeltaSourceState();
}

void UNetworkPhysicsComponent::ServerReceiveDeltaSourceStateFrame_Implementation(const uint32 Index, const int32 Frame)
{
	ensure(bIsUsingLegacyData);

	if (Index >= static_cast<uint32>(DeltaSourceStates.Num()))
	{
#if DEBUG_NETWORK_PHYSICS_DELTASERIALIZATION
		UE_LOGF(LogChaos, Log, "[DEBUG Delta Serialization] [SERVER]     Owner failed to acknowledged delta source STATE for frame: %d, index larger than collection: %d >= %d  --  Name: %ls", Frame, Index, DeltaSourceStates.Num(), *AActor::GetDebugName(GetOwner()));
#endif
		return;
	}

	TUniquePtr<FNetworkPhysicsData>& Data = DeltaSourceStates[Index];
	if (Data->ServerFrame == Frame)
	{
		// Set latest delta source index acknowledged by the client so that we can start using this delta source
		LatestAcknowledgedDeltaSourceStateIndex = Index;

#if DEBUG_NETWORK_PHYSICS_DELTASERIALIZATION
		UE_LOGF(LogChaos, Log, "[DEBUG Delta Serialization] [SERVER]     Owner acknowledged delta source STATE frame: %d at index: %d  --  Name: %ls", Frame, LatestAcknowledgedDeltaSourceStateIndex, *AActor::GetDebugName(GetOwner()));
#endif
	}
	else
	{
#if DEBUG_NETWORK_PHYSICS_DELTASERIALIZATION
		UE_LOGF(LogChaos, Log, "[DEBUG Delta Serialization] [SERVER]     Owner failed to acknowledged delta source STATE for Index: %d - AckFrame %d is not equal to CachedFrame: %d --  Name: %ls", LatestAcknowledgedDeltaSourceStateIndex, Frame, Data->ServerFrame, *AActor::GetDebugName(GetOwner()));
#endif
	}
}

void UNetworkPhysicsComponent::AddDeltaSourceState()
{
	// Get the data entry for the correct index in the data sources array
	const int32 Index = ReplicatedDeltaSourceState.Index;
	
	if (Index >= DeltaSourceStates.Num())
	{
		check(false);
		return;
	}
	
	FNetworkPhysicsData* PhysicsData = DeltaSourceStates[Index].Get();

	// Extract the data from the replicated DeltaSources property
	if (ReplicatedDeltaSourceState.History->ExtractData(ReplicatedDeltaSourceState.History->GetLatestFrame(), /*bResetSolver*/false, PhysicsData, /*bExactFrame*/true))
	{
		// The data is now extracted via PhysicsData and stored inside DeltaSourceStates

		if (!HasServerWorld())
		{
			// On the client, set the latest index (unlike for DeltaSourceInput this latest index is not used on the client since the client doesn't send states towards the server, but set the value for the future)
			LatestAcknowledgedDeltaSourceStateIndex = Index;

#if DEBUG_NETWORK_PHYSICS_DELTASERIALIZATION
			UE_LOGF(LogChaos, Log, "[DEBUG Delta Serialization] %ls Received delta source STATE for frame: %d at index: %d  --  Name: %ls", (IsLocallyControlled() ? TEXT("[AUTONOMOUS]") : TEXT("[SIMULATED] ")), ReplicatedDeltaSourceState.History->GetLatestFrame(), LatestAcknowledgedDeltaSourceStateIndex, *AActor::GetDebugName(GetOwner()));
#endif

			if (IsLocallyControlled())
			{
				// If this client is the one controlling this entity, send back an acknowledgment to the server we have received the delta source for ServerFrame
				ServerReceiveDeltaSourceStateFrame(Index, PhysicsData->ServerFrame);
			}
		}
		else if (IsLocallyControlled())
		{
			// If server is locally controlled, set the latest index directly, else wait for the owning client to send back ServerReceiveDeltaSourceStateFrame before the server starts to use this 
			LatestAcknowledgedDeltaSourceStateIndex = Index;
		}

#if DEBUG_NETWORK_PHYSICS_DELTASERIALIZATION
		if (HasServerWorld())
		{
			UE_LOGF(LogChaos, Log, "[DEBUG Delta Serialization] [SERVER]     Sent delta source STATE for frame: %d at index: %d  --  Name: %ls", ReplicatedDeltaSourceState.History->GetLatestFrame(), Index, *AActor::GetDebugName(GetOwner()));
		}
#endif

		LatestCachedDeltaSourceStateIndex = Index;
	}
}

FNetworkPhysicsData* UNetworkPhysicsComponent::GetDeltaSourceInput(const int32 Value, const bool bValueIsIndexElseFrame)
{
	TUniquePtr<FNetworkPhysicsData>* DataPtr = nullptr;
	if (Value == -1) // Latest
	{
		DataPtr = &DeltaSourceInputs[LatestAcknowledgedDeltaSourceInputIndex];
	}
	else if (Value == -2) // Default
	{
		DataPtr = &InputDataDefault_Legacy;
	}
	else if(bValueIsIndexElseFrame)
	{
		if (Value < DeltaSourceInputs.Num())
		{
			DataPtr = &DeltaSourceInputs[Value];
		}
	}
	else // Value is Frame
	{
		int32 Index = GetDeltaSourceIndexForFrame(Value);
		if (Index < DeltaSourceInputs.Num())
		{
			if (DeltaSourceInputs[Index]->ServerFrame == Value)
			{
				DataPtr = &DeltaSourceInputs[Index];
			}
		}
	}

	return DataPtr ? DataPtr->Get() : nullptr;
}

FNetworkPhysicsData* UNetworkPhysicsComponent::GetDeltaSourceState(const int32 Value, const bool bValueIsIndexElseFrame)
{
	TUniquePtr<FNetworkPhysicsData>* DataPtr = nullptr;
	if (Value == -1) // Latest
	{
		DataPtr = &DeltaSourceStates[LatestAcknowledgedDeltaSourceStateIndex];
	}
	else if (Value == -2) // Default
	{
		DataPtr = &StateDataDefault_Legacy;
	}
	else if (bValueIsIndexElseFrame)
	{
		if (Value < DeltaSourceStates.Num())
		{
			DataPtr = &DeltaSourceStates[Value];
		}
	}
	else // Value is Frame
	{
		int32 Index = GetDeltaSourceIndexForFrame(Value);
		if (Index < DeltaSourceStates.Num())
		{
			if (DeltaSourceStates[Index]->ServerFrame == Value)
			{
				DataPtr = &DeltaSourceStates[Index];
			}
		}
	}

	return DataPtr ? DataPtr->Get() : nullptr;
}

void UNetworkPhysicsComponent::OnRep_SetReplicatedStates()
{
	if (ReplicatedStates.bDeltaSerializationIssue)
	{
#if DEBUG_NETWORK_PHYSICS_DELTASERIALIZATION
		ensureMsgf(!IsLocallyControlled(), TEXT("OnRep_SetReplicatedStates failed delta serialization for locally controlled object, should not happen unless the pawn just got possessed."));
#endif
		return;
	}

	if (!NetworkPhysicsComponent_Internal || !StateHelper || !ReplicatedStates.History)
	{
		return;
	}
	ensure(bIsUsingLegacyData);

	if (FAsyncNetworkPhysicsComponentInput* AsyncInput = NetworkPhysicsComponent_Internal->GetProducerInputData_External())
	{
		if (!AsyncInput->StateData)
		{
			AsyncInput->StateData = StateHelper->CreateUniqueRewindHistory(ReplicatedStates.History->GetHistorySize());
		}

		ReplicatedStates.History->CopyAllDataGrowingOrdered(*AsyncInput->StateData.Get());
	}
}

void UNetworkPhysicsComponent::OnRep_SetReplicatedOwnerInputs()
{
	if (ReplicatedOwnerInputs.bDeltaSerializationIssue)
	{
#if DEBUG_NETWORK_PHYSICS_DELTASERIALIZATION
		ensureMsgf(!IsLocallyControlled(), TEXT("OnRep_SetReplicatedOwnerInputs failed delta serialization for locally controlled object, should not happen unless the pawn just got possessed."));
#endif
		return;
	}

	if (!NetworkPhysicsComponent_Internal || !InputHelper || !ReplicatedOwnerInputs.History)
	{
		return;
	}
	ensure(bIsUsingLegacyData);

	if (FAsyncNetworkPhysicsComponentInput* AsyncInput = NetworkPhysicsComponent_Internal->GetProducerInputData_External())
	{
		if (!AsyncInput->InputData)
		{
			AsyncInput->InputData = InputHelper->CreateUniqueRewindHistory(ReplicatedOwnerInputs.History->GetHistorySize());
		}

		ReplicatedOwnerInputs.History->CopyAllDataGrowingOrdered(*AsyncInput->InputData.Get());
	}
}

void UNetworkPhysicsComponent::OnRep_SetReplicatedRemoteInputs()
{
	if (ReplicatedRemoteInputs.bDeltaSerializationIssue)
	{
		return;
	}

	if (!NetworkPhysicsComponent_Internal || !InputHelper || !ReplicatedRemoteInputs.History)
	{
		return;
	}
	ensure(bIsUsingLegacyData);

	if (FAsyncNetworkPhysicsComponentInput* AsyncInput = NetworkPhysicsComponent_Internal->GetProducerInputData_External())
	{
		if (!AsyncInput->InputData)
		{
			AsyncInput->InputData = InputHelper->CreateUniqueRewindHistory(ReplicatedRemoteInputs.History->GetHistorySize());
		}

		ReplicatedRemoteInputs.History->CopyAllDataGrowingOrdered(*AsyncInput->InputData.Get());
	}
}

void UNetworkPhysicsComponent::ServerReceiveInputData_Implementation(const FNetworkPhysicsRewindDataInputProxy& ClientInputs)
{
	if (ClientInputs.bDeltaSerializationIssue)
	{
		ensureMsgf(false, TEXT("ServerReceiveInputData_Implementation failed delta serialization, should not happen on the server."));
		return;
	}

	if (!NetworkPhysicsComponent_Internal || !InputHelper || !ClientInputs.History)
	{
		return;
	}
	ensure(bIsUsingLegacyData);

	if (FAsyncNetworkPhysicsComponentInput* AsyncInput = NetworkPhysicsComponent_Internal->GetProducerInputData_External())
	{
		if (!AsyncInput->InputData)
		{
			AsyncInput->InputData = InputHelper->CreateUniqueRewindHistory(ClientInputs.History->GetHistorySize());
		}

		// Validate data in the received inputs
		if (bValidateDataOnGameThread && ActorComponent.IsValid())
		{
			ClientInputs.History->ValidateDataInHistory(ActorComponent.Get());
		}

		ClientInputs.History->CopyAllDataGrowingOrdered(*AsyncInput->InputData.Get());

		// Send received inputs to remote clients
		ReplicatedRemoteInputs.History->SetRecordDataIncremental(true); // Only record data that is newer than already cached data
		ClientInputs.History->CopyAllData(*ReplicatedRemoteInputs.History, /*bIncludeUnimportant*/ true, /*bIncludeImportant*/ true);
		MARK_PROPERTY_DIRTY_FROM_NAME(UNetworkPhysicsComponent, ReplicatedRemoteInputs, this);
	}
}

void UNetworkPhysicsComponent::ServerReceiveImportantInputData_Implementation(const FNetworkPhysicsRewindDataImportantInputProxy& ClientInputs)
{
	if (ClientInputs.bDeltaSerializationIssue)
	{
		ensureMsgf(false, TEXT("ServerReceiveImportantInputData_Implementation failed delta serialization, should not happen on the server."));
		return;
	}

	if (!NetworkPhysicsComponent_Internal || !ClientInputs.History)
	{
		return;
	}
	ensure(bIsUsingLegacyData);

	if (FAsyncNetworkPhysicsComponentInput* AsyncInput = NetworkPhysicsComponent_Internal->GetProducerInputData_External())
	{
		// Initialize received data since not all data is networked and when we clone this we expect to have fully initialized data
		ClientInputs.History->Initialize();

		// Validate data in the received inputs
		if (bValidateDataOnGameThread && ActorComponent.IsValid())
		{
			ClientInputs.History->ValidateDataInHistory(ActorComponent.Get());
		}

		// Create new data collection for marshaling
		AsyncInput->InputDataImportant.Add(ClientInputs.History->Clone());
	}
}

void UNetworkPhysicsComponent::MulticastReceiveImportantInputData_Implementation(const FNetworkPhysicsRewindDataImportantInputProxy& ServerInputs)
{
	// Ignore Multicast on server
	if (HasServerWorld())
	{
		return;
	}

	if (ServerInputs.bDeltaSerializationIssue)
	{
#if DEBUG_NETWORK_PHYSICS_DELTASERIALIZATION
		ensureMsgf(!IsLocallyControlled(), TEXT("MulticastReceiveImportantInputData_Implementation failed delta serialization for locally controlled object, should not happen unless the pawn just got possessed."));
#endif
		return;
	}

	if (!NetworkPhysicsComponent_Internal || !ServerInputs.History)
	{
		return;
	}
	ensure(bIsUsingLegacyData);

	if (FAsyncNetworkPhysicsComponentInput* AsyncInput = NetworkPhysicsComponent_Internal->GetProducerInputData_External())
	{
		// Initialize received data since not all data is networked and when we clone this we expect to have fully initialized data
		ServerInputs.History->Initialize();

		// Create new data collection for marshaling
		AsyncInput->InputDataImportant.Add(ServerInputs.History->Clone());
	}
}

void UNetworkPhysicsComponent::MulticastReceiveImportantStateData_Implementation(const FNetworkPhysicsRewindDataImportantStateProxy& ServerStates)
{
	// Ignore Multicast on server
	if (HasServerWorld())
	{
		return;
	}

	if (ServerStates.bDeltaSerializationIssue)
	{
#if DEBUG_NETWORK_PHYSICS_DELTASERIALIZATION
		ensureMsgf(!IsLocallyControlled(), TEXT("MulticastReceiveImportantStateData_Implementation failed delta serialization for locally controlled object, should not happen unless the pawn just got possessed."));
#endif
		return;
	}

	if (!NetworkPhysicsComponent_Internal || !ServerStates.History)
	{
		return;
	}
	ensure(bIsUsingLegacyData);

	if (FAsyncNetworkPhysicsComponentInput* AsyncInput = NetworkPhysicsComponent_Internal->GetProducerInputData_External())
	{
		// Initialize received data since not all data is networked and when we clone this we expect to have fully initialized data
		ServerStates.History->Initialize();

		// Create new data collection for marshaling
		AsyncInput->StateDataImportant.Add(ServerStates.History->Clone());
	}
}

// Server RPC to receive inputs from client
void UNetworkPhysicsComponent::ServerReceiveInputCollection_Implementation(const FNetworkPhysicsDataCollection& ClientInputCollection)
{
	ensure(bIsUsingLegacyData == false);

	if (!NetworkPhysicsComponent_Internal || !InputHelper || !ClientInputCollection.DataArray.Num())
	{
		return;
	}

	if (FAsyncNetworkPhysicsComponentInput* AsyncInput = NetworkPhysicsComponent_Internal->GetProducerInputData_External())
	{
		// Temporarily populate the LocalFrame value with ServerFrame in the inputs to have a value to use when marshaling the inputs from GT to PT, the correct LocalFrame value will be set on PT when calling History->ReceiveNewData
		ClientInputCollection.UpdateLocalFrameFromServerFrame(0);

		if (!AsyncInput->InputData)
		{
			AsyncInput->InputData = InputHelper->CreateUniqueRewindHistory(ClientInputCollection.DataArray.Num());
		}

		InputHelper->CopyDataGrowingOrdered(ClientInputCollection, AsyncInput->InputData.Get());

		// Validate Inputs
		if (ImplementationInterface_External)
		{
			InputHelper->ValidateData(AsyncInput->InputData.Get(), *ImplementationInterface_External);
		}

		// Send received inputs to remote clients
		ReplicatedRemoteInputCollection.DataArray.SetNum(InputsToNetwork_Simulated);
		InputHelper->CopyIncrementalData(AsyncInput->InputData.Get(), ReplicatedRemoteInputCollection);
		MARK_PROPERTY_DIRTY_FROM_NAME(UNetworkPhysicsComponent, ReplicatedRemoteInputCollection, this);
	}
}

void UNetworkPhysicsComponent::ReceiveActionCollectionData(const FNetworkPhysicsActionCollection& ActionCollection)
{
	if (!NetworkPhysicsComponent_Internal || ActionCollection.DataArray.Num() <= 0)
	{
		return;
	}

	if (FAsyncNetworkPhysicsComponentInput* AsyncInput = NetworkPhysicsComponent_Internal->GetProducerInputData_External())
	{
		for (const TInstancedStruct<FNetworkPhysicsActionPayload>& ActionInstance : ActionCollection.DataArray)
		{
			if (ActionInstance.IsValid())
			{
				AsyncInput->ReplicatedActionData.Add(ActionInstance);
			}
		}
	}
}

void UNetworkPhysicsComponent::ServerReceiveUnreliableActionCollection_Implementation(const FNetworkPhysicsActionCollection& ClientActionCollection)
{
	ReceiveActionCollectionData(ClientActionCollection);
}

void UNetworkPhysicsComponent::ServerReceiveReliableActionCollection_Implementation(const FNetworkPhysicsActionCollection& ClientActionCollection)
{
	ReceiveActionCollectionData(ClientActionCollection);
}

void UNetworkPhysicsComponent::MulticastReceiveReliableActionCollection_Implementation(const FNetworkPhysicsActionCollection& ServerActionCollection)
{
	// NetMulticast executes on the server too - skip re-ingesting our own actions, the server already has them in ConfirmedActions.
	if (HasServerWorld())
	{
		return;
	}
	ReceiveActionCollectionData(ServerActionCollection);
}

// repnotify for inputs on owner client
void UNetworkPhysicsComponent::OnRep_SetReplicatedInputCollection()
{
	ensure(bIsUsingLegacyData == false);
	if (!NetworkPhysicsComponent_Internal || !InputHelper || !ReplicatedInputCollection.DataArray.Num())
	{
		return;
	}

	if (FAsyncNetworkPhysicsComponentInput* AsyncInput = NetworkPhysicsComponent_Internal->GetProducerInputData_External())
	{
		// Temporarily populate the LocalFrame value with ServerFrame in the inputs to have a value to use when marshaling the inputs from GT to PT, the correct LocalFrame value will be set on PT when calling History->ReceiveNewData
		ReplicatedInputCollection.UpdateLocalFrameFromServerFrame(0);

		if (!AsyncInput->InputData)
		{
			AsyncInput->InputData = InputHelper->CreateUniqueRewindHistory(ReplicatedInputCollection.DataArray.Num());
		}

		InputHelper->CopyDataGrowingOrdered(ReplicatedInputCollection, AsyncInput->InputData.Get());
	}
}

// repnotify for inputs on remote clients
void UNetworkPhysicsComponent::OnRep_SetReplicatedRemoteInputCollection()
{
	ensure(bIsUsingLegacyData == false);
	if (!NetworkPhysicsComponent_Internal || !InputHelper || !ReplicatedRemoteInputCollection.DataArray.Num())
	{
		return;
	}

	if (FAsyncNetworkPhysicsComponentInput* AsyncInput = NetworkPhysicsComponent_Internal->GetProducerInputData_External())
	{
		// Temporarily populate the LocalFrame value with ServerFrame in the inputs to have a value to use when marshaling the inputs from GT to PT, the correct LocalFrame value will be set on PT when calling History->ReceiveNewData
		ReplicatedRemoteInputCollection.UpdateLocalFrameFromServerFrame(0);

		if (!AsyncInput->InputData)
		{
			AsyncInput->InputData = InputHelper->CreateUniqueRewindHistory(ReplicatedRemoteInputCollection.DataArray.Num());
		}

		InputHelper->CopyDataGrowingOrdered(ReplicatedRemoteInputCollection, AsyncInput->InputData.Get());
	}
}

// repnotify for the states on the client
void UNetworkPhysicsComponent::OnRep_SetReplicatedStateCollection()
{
	ensure(bIsUsingLegacyData == false);
	if (!NetworkPhysicsComponent_Internal || !StateHelper || !ReplicatedStateCollection.DataArray.Num())
	{
		return;
	}

	if (FAsyncNetworkPhysicsComponentInput* AsyncInput = NetworkPhysicsComponent_Internal->GetProducerInputData_External())
	{
		// Temporarily populate the LocalFrame value with ServerFrame in the states to have a value to use when marshaling the inputs from GT to PT, the correct LocalFrame value will be set on PT when calling History->ReceiveNewData
		ReplicatedStateCollection.UpdateLocalFrameFromServerFrame(0);

		if (!AsyncInput->StateData)
		{
			AsyncInput->StateData = StateHelper->CreateUniqueRewindHistory(ReplicatedStateCollection.DataArray.Num());
		}

		StateHelper->CopyDataGrowingOrdered(ReplicatedStateCollection, AsyncInput->StateData.Get());
	}
}

void UNetworkPhysicsComponent::OnRep_SetReplicatedActionsCollection()
{
	ReceiveActionCollectionData(ReplicatedActionsCollection);
}

void UNetworkPhysicsComponent::EnqueueImmediateActionInstance_External(TInstancedStruct<FNetworkPhysicsActionPayload>&& ActionInstance, const UObject* SourceObject, const bool bReliable)
{
	const uint32 SourceId = UE::NetworkPhysicsUtils::GetNetworkStableHash_External(SourceObject);
	ensureMsgf(SourceId > 0u, TEXT("EnqueueImmediateActionInstance_External could not find a network stable source to base an ID from, the object needs to either be replicated or baked in the level. Else it's recommended to pass in a custom unique SourceId as a parameter instead of the source object. Action: %ls  ---  Source: %ls")
		, ActionInstance.GetScriptStruct() ? *ActionInstance.GetScriptStruct()->GetName() : TEXT("UNKNOWN ACTION")
		, SourceObject ? *SourceObject->GetName() : TEXT("UNKNOWN SOURCE OBJECT"));

	EnqueueImmediateActionInstance_External(MoveTemp(ActionInstance), SourceId, bReliable);
}

void UNetworkPhysicsComponent::EnqueueImmediateActionInstance_External(TInstancedStruct<FNetworkPhysicsActionPayload>&& ActionInstance, const uint32 SourceId, const bool bReliable)
{
	EnqueueScheduledActionInstanceAtFrame_External(MoveTemp(ActionInstance), SourceId, -1, bReliable);
}

void UNetworkPhysicsComponent::EnqueueScheduledActionInstance_External(TInstancedStruct<FNetworkPhysicsActionPayload>&& ActionInstance, const UObject* SourceObject, const float DelaySeconds, const bool bReliable)
{
	const uint32 SourceId = UE::NetworkPhysicsUtils::GetNetworkStableHash_External(SourceObject);
	ensureMsgf(SourceId > 0u, TEXT("EnqueueScheduledActionInstance_External could not find a network stable source to base an ID from, the object needs to either be replicated or baked in the level. Else it's recommended to pass in a custom unique SourceId as a parameter instead of the source object. Action: %ls  ---  Source: %ls")
		, ActionInstance.GetScriptStruct() ? *ActionInstance.GetScriptStruct()->GetName() : TEXT("UNKNOWN ACTION")
		, SourceObject ? *SourceObject->GetName() : TEXT("UNKNOWN SOURCE OBJECT"));

	EnqueueScheduledActionInstance_External(MoveTemp(ActionInstance), SourceId, DelaySeconds, bReliable);
}

void UNetworkPhysicsComponent::EnqueueScheduledActionInstance_External(TInstancedStruct<FNetworkPhysicsActionPayload>&& ActionInstance, const uint32 SourceId, const float DelaySeconds, const bool bReliable)
{
	int32 LocalFrame = INDEX_NONE;
	if (UWorld* World = GetWorld())
	{
		if (FPhysScene* PhysScene = World->GetPhysicsScene())
		{
			if (Chaos::FPhysicsSolverBase* Solver = static_cast<Chaos::FPhysicsSolverBase*>(PhysScene->GetSolver()))
			{
				if (Solver->IsUsingFixedDt())
				{
					// Only apply a frame if we are actually delaying, else leave as INDEX_NONE which will queue up the action ASAP and handle edge-cases.
					const int32 DelayFrames = FMath::CeilToInt32(DelaySeconds / Solver->GetAsyncDeltaTime());
					if (DelayFrames > 0)
					{
						LocalFrame = Solver->GetMarshallingManager().GetInternalStep_External() + DelayFrames;
					}
				}
			}
		}
	}
	EnqueueScheduledActionInstanceAtFrame_External(MoveTemp(ActionInstance), SourceId, LocalFrame, bReliable);
}

void UNetworkPhysicsComponent::EnqueueScheduledActionInstanceAtFrame_External(TInstancedStruct<FNetworkPhysicsActionPayload>&& ActionInstance, const UObject* SourceObject, const int32 LocalFrame, const bool bReliable)
{
	const uint32 SourceId = UE::NetworkPhysicsUtils::GetNetworkStableHash_External(SourceObject);
	ensureMsgf(SourceId > 0u, TEXT("EnqueueScheduledActionInstanceAtFrame_External could not find a network stable source to base an ID from, the object needs to either be replicated or baked in the level. Else it's recommended to pass in a custom unique SourceId as a parameter instead of the source object. Action: %ls  ---  Source: %ls")
		, ActionInstance.GetScriptStruct() ? *ActionInstance.GetScriptStruct()->GetName() : TEXT("UNKNOWN ACTION")
		, SourceObject ? *SourceObject->GetName() : TEXT("UNKNOWN SOURCE OBJECT"));

	EnqueueScheduledActionInstanceAtFrame_External(MoveTemp(ActionInstance), SourceId, LocalFrame, bReliable);
}

void UNetworkPhysicsComponent::EnqueueScheduledActionInstanceAtFrame_External(TInstancedStruct<FNetworkPhysicsActionPayload>&& ActionInstance, const uint32 SourceId, const int32 LocalFrame, const bool bReliable)
{
	if (!NetworkPhysicsComponent_Internal)
	{
		return;
	}

	FNetworkPhysicsActionPayload& Action = ActionInstance.GetMutable<FNetworkPhysicsActionPayload>();
	Action.SourceId = SourceId;

	// If we have a LocalFrame passed in, convert that to ServerFrame, if INDEX_NONE set ServerFrame to that which indicates that the action should be applied ASAP
	Action.ServerFrame = (LocalFrame == INDEX_NONE) ? INDEX_NONE : (LocalFrame + UE::NetworkPhysicsUtils::GetNetworkPhysicsTickOffset_External(GetWorld()));

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	if (PhysicsReplicationCVars::ResimulationCVars::bActionsEnableDebugLogs)
	{
		UE_LOGF(LogChaos, Log, "GT %ls EnqueueActionAtFrame - Type: %ls - SourceId: %ld - LocalFrame: %d + Offset: %d = ServerFrame: %d"
			, (HasServerWorld() ? TEXT(" -- SERVER -- ") : TEXT(" ++ CLIENT ++ ")), (ActionInstance.GetScriptStruct() ? *ActionInstance.GetScriptStruct()->GetName() : TEXT("UNKNOWN ACTION"))
			, Action.SourceId, LocalFrame, UE::NetworkPhysicsUtils::GetNetworkPhysicsTickOffset_External(GetWorld()), Action.ServerFrame);
	}
#endif

	if (FAsyncNetworkPhysicsComponentInput* AsyncInput = NetworkPhysicsComponent_Internal->GetProducerInputData_External())
	{
		AsyncInput->LocalActionData.Add(MoveTemp(ActionInstance));
		AsyncInput->LocalActionDataReliable.Add(bReliable);
	}
}


bool UNetworkPhysicsComponent::HasServerWorld() const
{
	Chaos::EnsureIsInGameThreadContext();
	return GetWorld()->IsNetMode(NM_DedicatedServer) || GetWorld()->IsNetMode(NM_ListenServer);
}

bool UNetworkPhysicsComponent::IsLocallyControlled() const
{
	Chaos::EnsureIsInGameThreadContext();
	if (bIsRelayingLocalInputs)
	{
		return true;
	}
	
	if (const AController* Controller = GetController())
	{
		return Controller->IsLocalController();
	}
	
	return false;
}

bool UNetworkPhysicsComponent::IsNetworkPhysicsTickOffsetAssigned() const
{
	Chaos::EnsureIsInGameThreadContext();
	if (APlayerController* PlayerController = GetPlayerController())
	{
		return PlayerController->GetNetworkPhysicsTickOffsetAssigned();
	}
	return false;
}

void UNetworkPhysicsComponent::SetCompareStateToTriggerRewind(const bool bInCompareStateToTriggerRewind, const bool bInIncludeSimProxies)
{
	bCompareStateToTriggerRewind = bInCompareStateToTriggerRewind;
	bCompareStateToTriggerRewindIncludeSimProxies = bInIncludeSimProxies;
	if (NetworkPhysicsComponent_Internal)
	{
		if (FAsyncNetworkPhysicsComponentInput* AsyncInput = NetworkPhysicsComponent_Internal->GetProducerInputData_External())
		{
			AsyncInput->bCompareStateToTriggerRewind = bCompareStateToTriggerRewind;
			AsyncInput->bCompareStateToTriggerRewindIncludeSimProxies = bInIncludeSimProxies;
		}
	}
}

void UNetworkPhysicsComponent::SetCompareInputToTriggerRewind(const bool bInCompareInputToTriggerRewind)
{
	bCompareInputToTriggerRewind = bInCompareInputToTriggerRewind;
	if (NetworkPhysicsComponent_Internal)
	{
		if (FAsyncNetworkPhysicsComponentInput* AsyncInput = NetworkPhysicsComponent_Internal->GetProducerInputData_External())
		{
			AsyncInput->bCompareInputToTriggerRewind = bCompareInputToTriggerRewind;
		}
	}
}

APlayerController* UNetworkPhysicsComponent::GetPlayerController() const
{
	return Cast<APlayerController>(GetController());
}

AController* UNetworkPhysicsComponent::GetController() const
{
	Chaos::EnsureIsInGameThreadContext();
	if (AController* Controller = Cast<AController>(GetOwner()))
	{
		return Controller;
	}

	if (const APawn* Pawn = Cast<APawn>(GetOwner()))
	{
		if (AController* Controller = Pawn->GetController())
		{
			return Controller;
		}

		// In this case the AController can be found as the owner of the pawn
		if (AController* Controller = Cast<AController>(Pawn->GetOwner()))
		{
			return Controller;
		}
	}

	return nullptr;
}

void UNetworkPhysicsComponent::SetPhysicsObject(Chaos::FConstPhysicsObjectHandle InPhysicsObject)
{
	if (PhysicsObject == InPhysicsObject)
	{
		return;
	}

	PhysicsObject = InPhysicsObject;

	// Marshal data from Game Thread to Physics Thread
	if (NetworkPhysicsComponent_Internal)
	{
		if (FAsyncNetworkPhysicsComponentInput* AsyncInput = NetworkPhysicsComponent_Internal->GetProducerInputData_External())
		{
			AsyncInput->PhysicsObject = InPhysicsObject;
		}
	}
}

void UNetworkPhysicsComponent::SetActionHandler(INetworkPhysicsActionHandler_Internal* InActionHandler)
{
	ActionImplementationInterface_Internal = InActionHandler;

	if (NetworkPhysicsComponent_Internal)
	{
		if (FAsyncNetworkPhysicsComponentInput* AsyncInput = NetworkPhysicsComponent_Internal->GetProducerInputData_External())
		{
			AsyncInput->ActionHandler_Internal = InActionHandler;
		}
	}
}

void UNetworkPhysicsComponent::UpdateAsyncComponent(const bool bFullUpdate)
{
	// Marshal data from Game Thread to Physics Thread
	if (NetworkPhysicsComponent_Internal)
	{
		if (FAsyncNetworkPhysicsComponentInput* AsyncInput = NetworkPhysicsComponent_Internal->GetProducerInputData_External())
		{
			if (!HasServerWorld())
			{
				AsyncInput->InputsToNetwork_Owner = InputsToNetwork_Owner;
			}

			// bIsLocallyControlled is marshaled outside of the bFullUpdate because it's not always set when last bFullUpdate is called.
			AsyncInput->bIsLocallyControlled = IsLocallyControlled();

			if (bFullUpdate)
			{
				if (UWorld* World = GetWorld())
				{ 
					AsyncInput->NetMode = World->GetNetMode();
				}

				if (AActor* Owner = GetOwner())
				{ 
					AsyncInput->NetRole = Owner->GetLocalRole();
					AsyncInput->PhysicsReplicationMode = Owner->GetPhysicsReplicationMode();
					AsyncInput->ActorName = AActor::GetDebugName(Owner);
			
					UNetworkPhysicsSettingsComponent* SettingsComponent = Owner->FindComponentByClass<UNetworkPhysicsSettingsComponent>();
					if (SettingsComponent && SettingsComponent->GetSettings_Internal().IsValid())
					{
						AsyncInput->SettingsComponent = SettingsComponent->GetSettings_Internal();
					}
				}
				
				if (ActorComponent.IsValid())
				{
					AsyncInput->ActorComponent = ActorComponent;
				}

				AsyncInput->ImplementationInterface_Internal = ImplementationInterface_Internal;
				AsyncInput->ActionHandler_Internal = ActionImplementationInterface_Internal;
			}
		}
	}
}

void UNetworkPhysicsComponent::CreateAsyncDataHistory()
{
	if (NetworkPhysicsComponent_Internal)
	{
		if (FAsyncNetworkPhysicsComponentInput* AsyncInput = NetworkPhysicsComponent_Internal->GetProducerInputData_External())
		{
			if (InputHelper)
			{
				// Marshal the input helper to create both input data and input history on the physics thread
				AsyncInput->InputHelper = InputHelper->Clone();
			}

			if (StateHelper)
			{
				// Marshal the state helper to create both state data and state history on the physics thread
				AsyncInput->StateHelper = StateHelper->Clone();
			}
		}
	}
}

void UNetworkPhysicsComponent::RemoveDataHistory()
{
	// Tell the async network physics component to unregister from RewindData
	if (NetworkPhysicsComponent_Internal)
	{
		if (FAsyncNetworkPhysicsComponentInput* AsyncInput = NetworkPhysicsComponent_Internal->GetProducerInputData_External())
		{
			AsyncInput->bUnregisterDataHistoryFromRewindData = true;
		}
	}
}

void UNetworkPhysicsComponent::AddDataHistory()
{
	// Tell the async network physics component to register in RewindData
	if (NetworkPhysicsComponent_Internal)
	{
		if (FAsyncNetworkPhysicsComponentInput* AsyncInput = NetworkPhysicsComponent_Internal->GetProducerInputData_External())
		{
			AsyncInput->bRegisterDataHistoryInRewindData = true;
		}
	}
}

TSharedPtr<Chaos::FBaseRewindHistory>& UNetworkPhysicsComponent::GetStateHistory_Internal()
{
	if (NetworkPhysicsComponent_Internal)
	{
		return NetworkPhysicsComponent_Internal->StateHistory;
	}
	return StateHistory;
}

TSharedPtr<Chaos::FBaseRewindHistory>& UNetworkPhysicsComponent::GetInputHistory_Internal()
{
	if (NetworkPhysicsComponent_Internal)
	{
		return NetworkPhysicsComponent_Internal->InputHistory;
	}
	return InputHistory;
}


// --------------------------- Async Network Physics Component ---------------------------

// Initialize static
const FNetworkPhysicsSettingsNetworkPhysicsComponent FAsyncNetworkPhysicsComponent::SettingsNetworkPhysicsComponent_Default = FNetworkPhysicsSettingsNetworkPhysicsComponent();

FAsyncNetworkPhysicsComponent::FAsyncNetworkPhysicsComponent() : TSimCallbackObject()
	, bIsLocallyControlled(true)
	, NetMode(ENetMode::NM_Standalone)
	, NetRole(ENetRole::ROLE_Authority)
	, PhysicsReplicationMode(EPhysicsReplicationMode::Default)
	, bIsUsingLegacyData(false)
	, SettingsComponent(nullptr)
	, ActorComponent(nullptr)
	, ImplementationInterface_Internal(nullptr)
	, PhysicsObject(nullptr)
	, bCompareStateToTriggerRewind(false)
	, bCompareStateToTriggerRewindIncludeSimProxies(false)
	, bCompareInputToTriggerRewind(false)
{
}

void FAsyncNetworkPhysicsComponent::OnInitialize_Internal()
{
	if (Chaos::FPhysicsSolverBase* BaseSolver = GetSolver())
	{
		if (BaseSolver->IsNetworkPhysicsPredictionEnabled())
		{
			// Register for Pre- and Post- ProcessInputs_Internal callbacks
			if (FNetworkPhysicsCallback* SolverCallback = static_cast<FNetworkPhysicsCallback*>(BaseSolver->GetRewindCallback()))
			{
				DelegateOnPreProcessInputs_Internal = SolverCallback->PreProcessInputsInternal.AddRaw(this, &FAsyncNetworkPhysicsComponent::OnPreProcessInputs_Internal);
				DelegateOnPostProcessInputs_Internal = SolverCallback->PostProcessInputsInternal.AddRaw(this, &FAsyncNetworkPhysicsComponent::OnPostProcessInputs_Internal);
				DelegateOnAddResimulationRequest_Internal = SolverCallback->AddResimulationRequestInternal.AddRaw(this, &FAsyncNetworkPhysicsComponent::OnAddResimulationRequest_Internal);
			}

			if (Chaos::FRewindData* RewindData = BaseSolver->GetRewindData())
			{
				DelegateOnRewindDataResize_Internal = RewindData->RewindDataResize.AddRaw(this, &FAsyncNetworkPhysicsComponent::OnRewindDataResize_Internal);
			}
		}
		else
		{
			UE_LOGF(LogChaos, Warning, "A NetworkPhysicsComponent is trying to set up but 'Project Settings -> Physics -> Physics Prediction' is not enabled. The component might not work as intended.");
		}
	}
}

void FAsyncNetworkPhysicsComponent::OnUninitialize_Internal()
{
	if (Chaos::FPhysicsSolverBase* BaseSolver = GetSolver())
	{
		// Unregister for Pre- and Post- ProcessInputs_Internal callbacks
		if (FNetworkPhysicsCallback* SolverCallback = static_cast<FNetworkPhysicsCallback*>(BaseSolver->GetRewindCallback()))
		{
			SolverCallback->PreProcessInputsInternal.Remove(DelegateOnPreProcessInputs_Internal);
			DelegateOnPreProcessInputs_Internal.Reset();

			SolverCallback->PostProcessInputsInternal.Remove(DelegateOnPostProcessInputs_Internal);
			DelegateOnPostProcessInputs_Internal.Reset();

			SolverCallback->AddResimulationRequestInternal.Remove(DelegateOnAddResimulationRequest_Internal);
			DelegateOnAddResimulationRequest_Internal.Reset();
		}

		if (Chaos::FRewindData* RewindData = BaseSolver->GetRewindData())
		{
			RewindData->RewindDataResize.Remove(DelegateOnRewindDataResize_Internal);
			DelegateOnRewindDataResize_Internal.Reset();
		}

		if (SimDecaySettings.IsValid())
		{
			if (IPhysicsReplicationAsync* PhysRep = BaseSolver->GetPhysicsReplication_Internal())
			{
				PhysRep->RemoveParticleSimDecaySettings(SimDecayRegisteredHandle);
			}
			SimDecayRegisteredHandle = nullptr;
		}
	}

	UnregisterDataHistoryFromRewindData();
}

void FAsyncNetworkPhysicsComponent::OnPhysicsObjectUnregistered_Internal(Chaos::FConstPhysicsObjectHandle InPhysicsObject)
{
	if (PhysicsObject == InPhysicsObject)
	{
		UnregisterDataHistoryFromRewindData();
		PhysicsObject = nullptr;
	}
}

const FNetworkPhysicsSettingsNetworkPhysicsComponent& FAsyncNetworkPhysicsComponent::GetComponentSettings() const
{
	return SettingsComponent.IsValid() ? SettingsComponent.Pin()->NetworkPhysicsComponentSettings : SettingsNetworkPhysicsComponent_Default;
};

void FAsyncNetworkPhysicsComponent::ConsumeAsyncInput_Internal(const int32 PhysicsStep, const bool bTriggerResim)
{
	if (const FAsyncNetworkPhysicsComponentInput* AsyncInput = GetConsumerInput_Internal())
	{
		const FNetworkPhysicsSettingsNetworkPhysicsComponent& ComponentSettings = GetComponentSettings();

		/** Onetime setup marshaled data */
		{
			if (AsyncInput->SettingsComponent.IsSet() && (*AsyncInput->SettingsComponent).IsValid())
			{
				SettingsComponent = (*AsyncInput->SettingsComponent).Pin();
			}
			if (AsyncInput->bIsLocallyControlled.IsSet())
			{
				bIsLocallyControlled = *AsyncInput->bIsLocallyControlled;
			}
			if (AsyncInput->NetMode.IsSet())
			{
				NetMode = *AsyncInput->NetMode;
			}
			if (AsyncInput->NetRole.IsSet())
			{
				NetRole = *AsyncInput->NetRole;
			}
			if (AsyncInput->InputsToNetwork_Owner.IsSet())
			{
				// Only marshaled from GT to PT on the client
				InputsToNetwork_Owner = *AsyncInput->InputsToNetwork_Owner;
			}
			if (AsyncInput->PhysicsReplicationMode.IsSet())
			{
				PhysicsReplicationMode = *AsyncInput->PhysicsReplicationMode;
			}
			if (AsyncInput->ActorComponent.IsSet())
			{
				ActorComponent = *AsyncInput->ActorComponent;
			}
			if (AsyncInput->ImplementationInterface_Internal.IsSet())
			{
				ImplementationInterface_Internal = *AsyncInput->ImplementationInterface_Internal;
			}
			if (AsyncInput->ActionHandler_Internal.IsSet())
			{
				ActionHandler_Internal = *AsyncInput->ActionHandler_Internal;
			}
			if (AsyncInput->PhysicsObject.IsSet())
			{
				if (PhysicsObject == nullptr || PhysicsObject != *AsyncInput->PhysicsObject)
				{
					PhysicsObject = *AsyncInput->PhysicsObject;
					RegisterDataHistoryInRewindData();
				}
			}
			if (AsyncInput->ActorName.IsSet())
			{
				ActorName = *AsyncInput->ActorName;
			}
			if (AsyncInput->bRegisterDataHistoryInRewindData.IsSet())
			{
				RegisterDataHistoryInRewindData();
			}
			if (AsyncInput->bUnregisterDataHistoryFromRewindData.IsSet())
			{
				UnregisterDataHistoryFromRewindData();
			}
			if (AsyncInput->bCompareStateToTriggerRewind.IsSet())
			{
				bCompareStateToTriggerRewind = *AsyncInput->bCompareStateToTriggerRewind;
			}
			if (AsyncInput->bCompareStateToTriggerRewindIncludeSimProxies.IsSet())
			{
				bCompareStateToTriggerRewindIncludeSimProxies = *AsyncInput->bCompareStateToTriggerRewindIncludeSimProxies;
			}
			if (AsyncInput->bCompareInputToTriggerRewind.IsSet())
			{
				bCompareInputToTriggerRewind = *AsyncInput->bCompareInputToTriggerRewind;
			}

			const bool bStateHelperNeedsUpdate = AsyncInput->StateHelper.IsSet() && AsyncInput->StateHelper.GetValue().IsValid()
				&& (!StateHelper || StateHelper->GetTypeId() != AsyncInput->StateHelper.GetValue()->GetTypeId());

			const bool bInputHelperNeedsUpdate = AsyncInput->InputHelper.IsSet() && AsyncInput->InputHelper.GetValue().IsValid()
				&& (!InputHelper || InputHelper->GetTypeId() != AsyncInput->InputHelper.GetValue()->GetTypeId());

			if (bStateHelperNeedsUpdate || bInputHelperNeedsUpdate)
			{
				UnregisterDataHistoryFromRewindData();

				if (bStateHelperNeedsUpdate)
				{
					// Setup rewind data if not already done, and get history size
					const int32 NumFrames = SetupRewindData();

					StateHelper = (*AsyncInput->StateHelper)->Clone();

					// Create state history and local property
					StateData = StateHelper->CreateUniqueData();
					StateHistory = MakeShareable(StateHelper->CreateUniqueRewindHistory(NumFrames).Release());
					bIsUsingLegacyData = StateHelper->IsUsingLegacyData();
				}

				if (bInputHelperNeedsUpdate)
				{
					// Setup rewind data if not already done, and get history size
					const int32 NumFrames = SetupRewindData();

					InputHelper = (*AsyncInput->InputHelper)->Clone();

					// Create input history and local data properties
					InputData = InputHelper->CreateUniqueData();
					LatestInputReceiveData = InputHelper->CreateUniqueData();
					InputHistory = MakeShareable(InputHelper->CreateUniqueRewindHistory(NumFrames).Release());
					bIsUsingLegacyData = InputHelper->IsUsingLegacyData();
				}

				RegisterDataHistoryInRewindData();
			}
		}

		/** Continuously marshaled data */
		{
			const bool bIsServer = IsServer();
			const int32 NetworkPhysicsTickOffset = UE::NetworkPhysicsUtils::GetNetworkPhysicsTickOffset_Internal(GetSolver());

			/** Receive data helper */
			auto ReceiveHelper = [&](Chaos::FBaseRewindHistory* History, Chaos::FBaseRewindHistory* ReceiveData, const bool bImportant, const bool bCompareData)
				{
					const bool bCompareDataForRewind = bCompareData && !bIsServer && bTriggerResim;
					const int32 TryInjectAtFrame = bIsServer ? PhysicsStep : 0;
					const int32 ResimFrame = History->ReceiveNewData(*ReceiveData, (bIsServer ? 0 : NetworkPhysicsTickOffset), bCompareDataForRewind, bImportant, TryInjectAtFrame);
					if (bCompareDataForRewind)
					{
						TriggerResimulation(ResimFrame);
					}

#if !UE_BUILD_SHIPPING
					if (PhysicsReplicationCVars::ResimulationCVars::bDebugTriggerResimEveryNFrames > 0 && !bIsServer && IsLocallyControlled() && bTriggerResim)
					{
						int32 LatestReceivedFrame = ReceiveData->GetLatestFrame() - NetworkPhysicsTickOffset;
						if (PhysicsStep % PhysicsReplicationCVars::ResimulationCVars::bDebugTriggerResimEveryNFrames == 0)
						{
							TriggerResimulation(LatestReceivedFrame);
						}
					}
#endif // !UE_BUILD_SHIPPING

#if DEBUG_NETWORK_PHYSICS
					{
						FString NetRoleString = bIsServer ? FString("SERVER") : (IsLocallyControlled() ? FString("AUTONO") : FString("PROXY "));
						ReceiveData->DebugData(FString::Printf(TEXT("%s | PT | RECEIVE DATA | LatestFrame: %d | bImportant: %d | Name: %s"), *NetRoleString, ReceiveData->GetLatestFrame(), bImportant, *GetActorName()));
					}
#endif

					// Reset the received data after having consumed it
					ReceiveData->ResetFast();
				};

			const bool bCompareInput = ComponentSettings.GetCompareInputToTriggerRewind(bCompareInputToTriggerRewind) && IsLocallyControlled();
			const bool bCompareState = ComponentSettings.GetCompareStateToTriggerRewind(bCompareStateToTriggerRewind) && (IsLocallyControlled() || ComponentSettings.GetCompareStateToTriggerRewindIncludeSimProxies(bCompareStateToTriggerRewindIncludeSimProxies));

			// Receive Inputs
			if (AsyncInput->InputData && AsyncInput->InputData->HasDataInHistory())
			{
				// Validate data in the received inputs on the server
				if (bIsServer)
				{
					if (bIsUsingLegacyData)
					{
						if (!ComponentSettings.GetValidateDataOnGameThread() && ActorComponent.IsValid() && ActorComponent.Get()->IsBeingDestroyed() == false)
						{
							AsyncInput->InputData->ValidateDataInHistory(ActorComponent.Get());
						}
					}
					else
					{
						if (InputHelper && ImplementationInterface_Internal)
						{
							InputHelper->ValidateData(AsyncInput->InputData.Get(), *ImplementationInterface_Internal);
						}
					}
				}

				// If setting is true, request resimulation for sim-proxies when receiving inputs that are newer than latest already received input
				if (!bIsServer && ComponentSettings.GetTriggerResimOnInputReceive() && !IsLocallyControlled() && bTriggerResim)
				{
					const int32 LatestReceiveFrame = AsyncInput->InputData.Get()->GetLatestFrame() - NetworkPhysicsTickOffset;
					const int32 NextInputFrame = InputHistory.Get()->GetLatestFrame() + 1;
					if (LatestReceiveFrame >= NextInputFrame)
					{
						const int32 EarliestReceivedFrame = AsyncInput->InputData.Get()->GetEarliestFrame() - NetworkPhysicsTickOffset;
						const int32 EarliestNewInputFrame = FMath::Max(EarliestReceivedFrame, NextInputFrame);
						TriggerResimulation(EarliestNewInputFrame);
					}
				}

				ReceiveHelper(InputHistory.Get(), AsyncInput->InputData.Get(), /*bImportant*/false, bCompareInput);
			}

			// Receive States
			if (AsyncInput->StateData && AsyncInput->StateData->HasDataInHistory())
			{
				LastReceivedStateFrame = FMath::Max(LastReceivedStateFrame, AsyncInput->StateData->GetLatestFrame() - NetworkPhysicsTickOffset);
				ReceiveHelper(StateHistory.Get(), AsyncInput->StateData.Get(), /*bImportant*/false, bCompareState);
			}

			// Receive Important Inputs
			for (const TUniquePtr<Chaos::FBaseRewindHistory>& InputImportant : AsyncInput->InputDataImportant)
			{
				if (!InputImportant || !InputImportant->HasDataInHistory())
				{
					continue;
				}
				ReceiveHelper(InputHistory.Get(), InputImportant.Get(), /*bImportant*/true, bCompareInput);
			}

			// Receive Important States
			for (const TUniquePtr<Chaos::FBaseRewindHistory>& StateImportant : AsyncInput->StateDataImportant)
			{
				if (!StateImportant || !StateImportant->HasDataInHistory())
				{
					continue;
				}
				LastReceivedStateFrame = FMath::Max(LastReceivedStateFrame, StateImportant->GetLatestFrame() - NetworkPhysicsTickOffset);
				ReceiveHelper(StateHistory.Get(), StateImportant.Get(), /*bImportant*/true, bCompareState);
			}

			// Running average of how far behind the forward predicted timeline we receive inputs
			if (Chaos::FPBDRigidsSolver* RigidsSolver = GetRigidSolver())
			{
				if (!bIsServer && !IsLocallyControlled() && InputHistory)
				{
					const float Dt = RigidsSolver->GetAsyncDeltaTime();
					const int32 InputPredictedFrames = FMath::Max(RigidsSolver->GetCurrentFrame() - InputHistory->GetLatestFrame(), 0);

					const float AverageOverSeconds = PhysicsReplicationCVars::ResimulationCVars::SimDecayNetPhysicsCompAverageOverTime;
					const float AverageAlpha = FMath::Clamp(Dt / (Dt + AverageOverSeconds), 0.0f, 1.0f);
					InputPredictionFramesAverage += (static_cast<float>(InputPredictedFrames) - InputPredictionFramesAverage) * AverageAlpha;
				}
			}

			// AsyncInput can be used multiple frames, in a row or during resim, but we should only process incoming actions one time
			// Get a mutable version and clear the action arrays after processing them
			FAsyncNetworkPhysicsComponentInput* MutableAsyncInput = const_cast<FAsyncNetworkPhysicsComponentInput*>(AsyncInput);

			// Receive Replicated Actions
			if (AsyncInput->ReplicatedActionData.Num() > 0)
			{
				for (const TInstancedStruct<FNetworkPhysicsActionPayload>& ActionInstance : AsyncInput->ReplicatedActionData)
				{
					RegisterAction_Internal(ActionInstance, /*bWasReceivedFromReplication*/ true);
				}
				MutableAsyncInput->ReplicatedActionData.Reset();
			}

			// Receive Local Actions
			if (AsyncInput->LocalActionData.Num() > 0)
			{
				for (int32 ActionIdx = 0; ActionIdx < AsyncInput->LocalActionData.Num(); ++ActionIdx)
				{
					const bool bActionReliable = AsyncInput->LocalActionDataReliable.IsValidIndex(ActionIdx) ? AsyncInput->LocalActionDataReliable[ActionIdx] : false;
					RegisterAction_Internal(AsyncInput->LocalActionData[ActionIdx], /*bWasReceivedFromReplication*/ false, bActionReliable);
				}
				MutableAsyncInput->LocalActionData.Reset();
				MutableAsyncInput->LocalActionDataReliable.Reset();
			}
		}
	}
}

void FAsyncNetworkPhysicsComponent::RegisterAction_Internal(
	const TInstancedStruct<FNetworkPhysicsActionPayload>& NewActionInstance,
	const bool bWasReceivedFromReplication,
	const bool bReliable)
{
	using EActionAuthorStyle = FNetworkPhysicsActionPayload::EActionAuthorStyle;

	if (!NewActionInstance.IsValid() || !GetRigidSolver() || !GetEvolution())
	{
		return;
	}

	if (ConfirmedActions.IsInitialized() == false || PredictedActions.IsInitialized() == false || PendingProposedActions.IsInitialized() == false)
	{
		const int32 NumFrames = SetupRewindData();
		ConfirmedActions.Initialize(NumFrames);
		PredictedActions.Initialize(NumFrames);
		PendingProposedActions.Initialize(NumFrames);
	}

	// Helper function to get the LocalFrame for an action, if an action does not have a ServerFrame set that means it should be applied ASAP, i.e. use current LocalFrame.
	auto GetActionLocalFrameHelper = [&](const FNetworkPhysicsActionPayload& Action, const bool bWasReceivedFromReplication, const int32 CurrentLocalFrame, const int32 TickOffset) -> int32
		{
			int32 ActionLocalFrame = Action.ServerFrame != INDEX_NONE ? (Action.ServerFrame - TickOffset) : CurrentLocalFrame;

			// Local actions can be enqueued after this frame has already applied actions, LastAppliedActionLocalFrame gets set when we reach the point where we apply actions during the frame
			// In that case they need to get registered for the next frame instead of the current one.
			if (bWasReceivedFromReplication == false && Action.ServerFrame == INDEX_NONE && LastAppliedActionLocalFrame >= ActionLocalFrame)
			{
				ActionLocalFrame = LastAppliedActionLocalFrame + 1;
			}

			return ActionLocalFrame;
		};

	// General Properties
	const bool bServer = IsServer();
	const bool bLocalControl = IsLocallyControlled();
	const bool bIsResim = GetEvolution()->IsResimming();
	const int32 CurrentLocalFrame = GetRigidSolver()->GetCurrentFrame();
	const int32 TickOffset = UE::NetworkPhysicsUtils::GetNetworkPhysicsTickOffset_Internal(GetSolver());

	// Action Properties
	const FNetworkPhysicsActionPayload& NewAction = FActionHistory::GetRef(NewActionInstance);
	const int32 ActionLocalFrame = GetActionLocalFrameHelper(NewAction, bWasReceivedFromReplication, CurrentLocalFrame, TickOffset);
	const EActionAuthorStyle AuthorStyle = NewAction.GetAuthorStyle();
	const bool bIsProposedStyle = AuthorStyle == EActionAuthorStyle::Proposed || AuthorStyle == EActionAuthorStyle::ProposedAutonomousOnly;
	const bool bIsAutonomousOnlyStyle = AuthorStyle == EActionAuthorStyle::PredictedAutonomousOnly || AuthorStyle == EActionAuthorStyle::ProposedAutonomousOnly;

	// Proposal reuse should be exact on forward-simulated client prediction, but allowed a small frame window during resim and on the server when adopting proposals.
	const int32 ProposalFrameWindow = (bIsResim || bServer) ? FMath::Max(PhysicsReplicationCVars::ResimulationCVars::bActionsEquivalenceFrameWindow, 0) : 0;

	// Confirmed actions should suppress equivalent local prediction in a small window.
	const int32 PredictionFrameWindow = FMath::Max(PhysicsReplicationCVars::ResimulationCVars::bActionsEquivalenceFrameWindow, 0);

	if (bWasReceivedFromReplication)
	{
		if (bServer) // Server received replicated action
		{
			// Server only accepts replicated action proposals
			if (bIsProposedStyle == false)
			{
				ensure(false);
				return;
			}

			// Proposed actions should already carry a client-generated ActionId.
			if (NewAction.ActionId == 0u)
			{
				ensure(false);
				return;
			}

			// Ignore duplicate client sends of the same proposal.
			if (PendingProposedActions.FindActiveActionById(NewAction.ActionId, /*bIncludeApplied*/true).IsValid())
			{
				return;
			}

			// Ignore proposal that has already been confirmed as an authoritative action.
			if (ConfirmedActions.FindActiveActionById(NewAction.ActionId, /*bIncludeApplied*/true).IsValid())
			{
				return;
			}

			// Store the proposal exactly as received.
			PendingProposedActions.AddAction(NewActionInstance, ActionLocalFrame, TickOffset, /*OverrideActionId*/0u, /*bClearActionId=*/false, bReliable);

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
			if (PhysicsReplicationCVars::ResimulationCVars::bActionsEnableDebugLogs)
			{
				UE_LOGF(LogChaos, Log, "PT %ls RegisterAction_Internal - PendingProposedActions.AddAction - Server Receive Proposal - ActionId: %d - ServerFrame: %d - bIsResim: %d", (bServer ? TEXT(" -- SERVER -- ") : TEXT(" ++ CLIENT ++ ")), NewActionInstance.Get().ActionId, (ActionLocalFrame + TickOffset), bIsResim);
			}
#endif
			return;
		}
		else // Client received replicated action
		{
			// Incoming replicated actions on the client from the server are authoritative confirmed actions.

			// Store the confirmed action exactly as received.
			ConfirmedActions.AddAction(NewActionInstance, ActionLocalFrame, TickOffset, /*OverrideActionId*/0u, /*bClearActionId=*/false, bReliable);

			// Retire the predicted actions for this source within the leniency window. Per-frame atomicity: if any confirmed for this source arrives, the server emitted the complete set, so any client predictions for the same source on those frames are speculative-only and must be retired regardless of payload divergence.
			const bool bPredictedActionDeactivated = PredictedActions.DeactivateEquivalentActionsInRange(NewActionInstance, (ActionLocalFrame - PredictionFrameWindow), (ActionLocalFrame + PredictionFrameWindow), /*bCheckSourceEquivalent*/true);

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
			if (PhysicsReplicationCVars::ResimulationCVars::bActionsEnableDebugLogs)
			{
				UE_LOGF(LogChaos, Log, "PT %ls RegisterAction_Internal - ConfirmedActions.AddAction - Client Receive Confirmed - ActionId: %d - ServerFrame : %d - bIsResim: %d", (bServer ? TEXT(" --SERVER-- ") : TEXT(" ++CLIENT++ ")), NewActionInstance.Get().ActionId, (ActionLocalFrame + TickOffset), bIsResim);
				UE_LOGF(LogChaos, Log, "	PT %ls RegisterAction_Internal - PredictedActions.DeactivateEquivalentActionsInRange - Frames %d -> %d - Succeeded: %d - bIsResim: %d", (bServer ? TEXT(" -- SERVER -- ") : TEXT(" ++ CLIENT ++ ")), (ActionLocalFrame - PredictionFrameWindow), (ActionLocalFrame + PredictionFrameWindow), bPredictedActionDeactivated, bIsResim);
			}
#endif


			if (bLocalControl == true && bIsProposedStyle == true && NewAction.ActionId != 0u)
			{
				// The autonomous proxy proposal has come back from the server as authority.
				// Retire the pending proposal.
				const bool bProposedActionDeactivated = PendingProposedActions.DeactivateAllByActionId(NewAction.ActionId);

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
				if (PhysicsReplicationCVars::ResimulationCVars::bActionsEnableDebugLogs)
				{
					UE_LOGF(LogChaos, Log, "	PT %ls RegisterAction_Internal - PendingProposedActions.DeactivateAllByActionId - ActionId: %d - Succeeded: %d - bIsResim: %d", (bServer ? TEXT(" -- SERVER -- ") : TEXT(" ++ CLIENT ++ ")), NewAction.ActionId, bProposedActionDeactivated, bIsResim);
				}
#endif
			}

			return;
		}
	}
	else /*Locally Produced Action*/
	{
		if (bServer) // Server locally produced action
		{
			if (bIsProposedStyle == true)
			{
				bool bRejectedProposal = false;

				// Check for a matching proposed action from the client
				FActionHistoryLocation MatchingProposalLocation = PendingProposedActions.FindEquivalentActionInRange(NewActionInstance, (ActionLocalFrame - ProposalFrameWindow), (ActionLocalFrame + ProposalFrameWindow), /*bIncludeApplied*/false, /*bCheckSourceEquivalent*/false);
				
				// If we did not find an equivalent action, look for an action that matches type and source only and stomp the action data but keep the ActionId
				if (MatchingProposalLocation.IsValid() == false)
				{
					MatchingProposalLocation = PendingProposedActions.FindEquivalentActionInRange(NewActionInstance, (ActionLocalFrame - ProposalFrameWindow), (ActionLocalFrame + ProposalFrameWindow), /*bIncludeApplied*/false, /*bCheckSourceEquivalent*/true);
					bRejectedProposal = true;
				}

				if (MatchingProposalLocation.IsValid())
				{
					const TInstancedStruct<FNetworkPhysicsActionPayload>* ProposedActionInstance = PendingProposedActions.GetAction(MatchingProposalLocation);

					if (ProposedActionInstance != nullptr)
					{
						if (bRejectedProposal)
						{
							// Override proposal but keep ActionId
							ConfirmedActions.AddAction(NewActionInstance, ActionLocalFrame, TickOffset, ProposedActionInstance->Get().ActionId, /*bClearActionId=*/false, bReliable);

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
							if (PhysicsReplicationCVars::ResimulationCVars::bActionsEnableDebugLogs)
							{
								UE_LOGF(LogChaos, Log, "PT %ls RegisterAction_Internal - ConfirmedActions.AddAction - Server Rejected Proposal - ActionId: %d - ServerFrame: %d - bIsResim: %d", (bServer ? TEXT(" -- SERVER -- ") : TEXT(" ++ CLIENT ++ ")), ProposedActionInstance->Get().ActionId, (ActionLocalFrame + TickOffset), bIsResim);
							}
#endif
						}
						else // Accepted proposal
						{
							const int32 ProposedLocalFrame = PendingProposedActions.GetLocalFrame(MatchingProposalLocation);

							// Store the confirmed proposed action exactly as received from client (keeping the ActionId that the client set)
							ConfirmedActions.AddAction(*ProposedActionInstance, ProposedLocalFrame, TickOffset, /*OverrideActionId*/0u, /*bClearActionId=*/false, bReliable);

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
							if (PhysicsReplicationCVars::ResimulationCVars::bActionsEnableDebugLogs)
							{
								UE_LOGF(LogChaos, Log, "PT %ls RegisterAction_Internal - ConfirmedActions.AddAction - Server Confirmed Proposal - ActionId: %d - ServerFrame: %d - bIsResim: %d", (bServer ? TEXT(" -- SERVER -- ") : TEXT(" ++ CLIENT ++ ")), ProposedActionInstance->Get().ActionId, (ProposedLocalFrame + TickOffset), bIsResim);
							}
#endif
						}

						const bool bProposedActionDeleted = PendingProposedActions.Deactivate(MatchingProposalLocation);

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
						if (PhysicsReplicationCVars::ResimulationCVars::bActionsEnableDebugLogs)
						{
							UE_LOGF(LogChaos, Log, "	PT %ls RegisterAction_Internal - PendingProposedActions.Deactivate - Succeeded: %d - bIsResim: %d", (bServer ? TEXT(" -- SERVER -- ") : TEXT(" ++ CLIENT ++ ")), bProposedActionDeleted, bIsResim);
						}
#endif
						return;
					}
				}
			}

			// Store the confirmed action and set a unique ActionId
			const uint32 NewActionId = GetUniqueActionId_Internal();
			ConfirmedActions.AddAction(NewActionInstance, ActionLocalFrame, TickOffset, NewActionId, /*bClearActionId=*/false, bReliable);
			
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
			if (PhysicsReplicationCVars::ResimulationCVars::bActionsEnableDebugLogs)
			{
				UE_LOGF(LogChaos, Log, "PT %ls RegisterAction_Internal - ConfirmedActions.AddAction - Server Local Action - ActionId: %d - ServerFrame: %d - bIsResim: %d", (bServer ? TEXT(" -- SERVER -- ") : TEXT(" ++ CLIENT ++ ")), NewActionId, (ActionLocalFrame + TickOffset), bIsResim);
			}
#endif
			return;
		}
		else // Client locally produced action
		{
			const bool bCanPredictOnThisClient = AuthorStyle != EActionAuthorStyle::Authority && (bLocalControl || bIsAutonomousOnlyStyle == false);

			if (bCanPredictOnThisClient == false)
			{
				return;
			}

			// If the server has already established equivalent authority for this frame or just ahead of it, do not add another local prediction.
			if (ConfirmedActions.HasEquivalentActionInRange(NewActionInstance, ActionLocalFrame, (ActionLocalFrame + PredictionFrameWindow)))
			{
				return;
			}

			if (bIsProposedStyle == true && bLocalControl == true)
			{
				// Autonomous-proxy proposing an action

				if (bIsResim)
				{
					// During resim, reuse the existing pending proposal if the same logical action is regenerated within the allowed frame window.
					// Skip proposals already claimed this resim pass so multiple same-source actions each bind to a distinct proposal.
					const FActionHistoryLocation ExistingPendingProposalLocation = PendingProposedActions.FindEquivalentActionInRange(NewActionInstance, (ActionLocalFrame - ProposalFrameWindow), (ActionLocalFrame + ProposalFrameWindow), /*bIncludeApplied*/false, /*bCheckSourceEquivalent*/true, /*bSkipClaimedThisResim*/true);

					if (ExistingPendingProposalLocation.IsValid())
					{
						const TInstancedStruct<FNetworkPhysicsActionPayload>* ExistingPendingProposal = PendingProposedActions.GetAction(ExistingPendingProposalLocation);

						if (ExistingPendingProposal != nullptr)
						{
							// Found a matching proposal, claim it so a follow-up same-source action finds another unclaimed proposal
							PendingProposedActions.MarkClaimed(ExistingPendingProposalLocation);

							const int32 ExistingPendingProposalLocalFrame = PendingProposedActions.GetLocalFrame(ExistingPendingProposalLocation);

							// Try to claim a matching unapplied predicted entry from a prior pass within the leniency window so multiple resims of the producing frame do not stack duplicate unapplied predictions on the target frame. Source-equivalent so payload divergence across resim passes does not split the same logical action.
							const FActionHistoryLocation ExistingPredictedLocation = PredictedActions.FindEquivalentActionInRange(*ExistingPendingProposal, (ExistingPendingProposalLocalFrame - PredictionFrameWindow), (ExistingPendingProposalLocalFrame + PredictionFrameWindow), /*bIncludeApplied*/false, /*bCheckSourceEquivalent*/true, /*bSkipClaimedThisResim*/true);

							if (ExistingPredictedLocation.IsValid())
							{
								// Found a matching predicted action, claim it so a follow-up same-source action finds another unclaimed predicted action
								PredictedActions.MarkClaimed(ExistingPredictedLocation);

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
								if (PhysicsReplicationCVars::ResimulationCVars::bActionsEnableDebugLogs)
								{
									UE_LOGF(LogChaos, Log, "PT %ls RegisterAction_Internal - PredictedActions.MarkClaimed - Client Reuse Predicted - ActionId: %d - ServerFrame : %d - bIsResim: %d", (bServer ? TEXT(" --SERVER-- ") : TEXT(" ++CLIENT++ ")), ExistingPendingProposal->Get().ActionId, (PredictedActions.GetLocalFrame(ExistingPredictedLocation) + TickOffset), bIsResim);
								}
#endif
							}
							else // No matching unclaimed predicted action found
							{
								// Store the equivalent proposed action as predicted and clear ActionId since no predicted action has an ActionId.
								PredictedActions.AddAction(*ExistingPendingProposal, ExistingPendingProposalLocalFrame, TickOffset, /*OverrideActionId*/0u, /*bClearActionId=*/true);

	#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
								if (PhysicsReplicationCVars::ResimulationCVars::bActionsEnableDebugLogs)
								{
									UE_LOGF(LogChaos, Log, "PT %ls RegisterAction_Internal - PredictedActions.AddAction - Client Reuse Proposal - ActionId: %d - ServerFrame : %d - bIsResim: %d", (bServer ? TEXT(" --SERVER-- ") : TEXT(" ++CLIENT++ ")), ExistingPendingProposal->Get().ActionId, (ExistingPendingProposalLocalFrame + TickOffset), bIsResim);
								}
	#endif
							}
							return;
						}
					}
					// No matching unclaimed proposal found
				}

				// Store the proposal action and set a unique ActionId
				const uint32 NewActionId = GetUniqueActionId_Internal();
				PendingProposedActions.AddAction(NewActionInstance, ActionLocalFrame, TickOffset, NewActionId, /*bClearActionId=*/false, bReliable);

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
				if (PhysicsReplicationCVars::ResimulationCVars::bActionsEnableDebugLogs)
				{
					UE_LOGF(LogChaos, Log, "	PT %ls RegisterAction_Internal - PendingProposedActions.AddAction - Client Local Proposal - ActionId: %d - ServerFrame: %d - bIsResim: %d", (bServer ? TEXT(" -- SERVER -- ") : TEXT(" ++ CLIENT ++ ")), NewActionId, (ActionLocalFrame + TickOffset), bIsResim);
				}
#endif
			}

			if (bIsResim)
			{
				// Try to claim a matching unapplied predicted entry from a prior pass within the leniency window so multiple resims of the producing frame do not stack duplicate unapplied predictions on the target frame. Source-equivalent so payload divergence across resim passes does not split the same logical action.
				const FActionHistoryLocation ExistingPredictedLocation = PredictedActions.FindEquivalentActionInRange(NewActionInstance, (ActionLocalFrame - PredictionFrameWindow), (ActionLocalFrame + PredictionFrameWindow), /*bIncludeApplied*/false, /*bCheckSourceEquivalent*/true, /*bSkipClaimedThisResim*/true);

				if (ExistingPredictedLocation.IsValid())
				{
					PredictedActions.MarkClaimed(ExistingPredictedLocation);

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
					if (PhysicsReplicationCVars::ResimulationCVars::bActionsEnableDebugLogs)
					{
						UE_LOGF(LogChaos, Log, "PT %ls RegisterAction_Internal - PredictedActions.MarkClaimed - Client Reuse Predicted - ActionId: %d - ServerFrame: %d - bIsResim: %d", (bServer ? TEXT(" -- SERVER -- ") : TEXT(" ++ CLIENT ++ ")), NewActionInstance.Get().ActionId, (PredictedActions.GetLocalFrame(ExistingPredictedLocation) + TickOffset), bIsResim);
					}
#endif
					return;
				}
			}

			// Store the predicted action.
			PredictedActions.AddAction(NewActionInstance, ActionLocalFrame, TickOffset, /*OverrideActionId*/0u, /*bClearActionId=*/true);

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
			if (PhysicsReplicationCVars::ResimulationCVars::bActionsEnableDebugLogs)
			{
				UE_LOGF(LogChaos, Log, "PT %ls RegisterAction_Internal - PredictedActions.AddAction - Client Local Prediction - ActionId: %d - ServerFrame: %d - bIsResim: %d", (bServer ? TEXT(" -- SERVER -- ") : TEXT(" ++ CLIENT ++ ")), NewActionInstance.Get().ActionId, (ActionLocalFrame + TickOffset), bIsResim);
			}
#endif
			return;
		}
	}
}

void FAsyncNetworkPhysicsComponent::ApplyActionsForFrame(const int32 LocalFrame)
{
	if (!ActionHandler_Internal || !GetRigidSolver() || !GetEvolution())
	{
		return;
	}

	if (ConfirmedActions.IsInitialized() == false || PredictedActions.IsInitialized() == false)
	{
		return;
	}

	const bool bIsResim = GetEvolution()->IsResimming();

	// From this point onward, any new enqueued local action should land on the next frame.
	LastAppliedActionLocalFrame = LocalFrame;

	// Apply authoritative confirmed actions for this exact local frame.
	FActionHistoryFrameData* ConfirmedFrameData = ConfirmedActions.FindFrameData(LocalFrame);
	if (ConfirmedFrameData != nullptr)
	{
		for (FActionHistoryEntry& ConfirmedActionEntry : ConfirmedFrameData->Entries)
		{
			if (FActionHistory::IsUsableEntry(ConfirmedActionEntry, /*bAllowApplied*/bIsResim) == false)
			{
				continue;
			}

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
			if (PhysicsReplicationCVars::ResimulationCVars::bActionsEnableDebugLogs)
			{
				UE_LOGF(LogChaos, Log, "PT %ls ApplyActionsForFrame - Apply Confirmed Action - Type: %ls - ServerFrame: %d - bIsResim: %d"
					, (IsServer() ? TEXT(" -- SERVER -- ") : TEXT(" ++ CLIENT ++ "))
					, ConfirmedActionEntry.ActionInstance.GetScriptStruct() ? *ConfirmedActionEntry.ActionInstance.GetScriptStruct()->GetName() : TEXT("UNKNOWN ACTION")
					, (LocalFrame + UE::NetworkPhysicsUtils::GetNetworkPhysicsTickOffset_Internal(GetSolver())), bIsResim);
			}
#endif

			ActionHandler_Internal->ApplyAction_Internal(ConfirmedActionEntry.ActionInstance);

			ConfirmedActionEntry.bApplied = true;
		}
	}

	// Apply local predicted actions for this exact local frame.
	FActionHistoryFrameData* PredictedFrameData = PredictedActions.FindFrameData(LocalFrame);
	if (PredictedFrameData != nullptr)
	{
		for (FActionHistoryEntry& PredictedActionEntry : PredictedFrameData->Entries)
		{
			if (FActionHistory::IsUsableEntry(PredictedActionEntry, /*bAllowApplied*/false) == false)
			{
				continue;
			}

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
			if (PhysicsReplicationCVars::ResimulationCVars::bActionsEnableDebugLogs)
			{
				UE_LOGF(LogChaos, Log, "PT %ls ApplyActionsForFrame - Apply Predicted Action - Type: %ls - ServerFrame: %d - bIsResim: %d"
					, (IsServer() ? TEXT(" -- SERVER -- ") : TEXT(" ++ CLIENT ++ "))
					, PredictedActionEntry.ActionInstance.GetScriptStruct() ? *PredictedActionEntry.ActionInstance.GetScriptStruct()->GetName() : TEXT("UNKNOWN ACTION")
					, (LocalFrame + UE::NetworkPhysicsUtils::GetNetworkPhysicsTickOffset_Internal(GetSolver())), bIsResim);
			}
#endif

			ActionHandler_Internal->ApplyAction_Internal(PredictedActionEntry.ActionInstance);

			PredictedActionEntry.bApplied = true;
			PredictedActionEntry.bActive = false;
		}
	}
}

uint16 FAsyncNetworkPhysicsComponent::GetUniqueActionId_Internal()
{
	// Lower 15 bits are the ID sequence, top bit identifies server/client origin.
	constexpr uint16 ServerBit = static_cast<uint16>(1u) << 15; // 1000 0000 0000 0000
	constexpr uint16 SequenceMask = static_cast<uint16>(~ServerBit); // 0111 1111 1111 1111

	// Increment the ActionIdIterator and only keep the first 15 bits
	ActionIdIterator = static_cast<uint16>(++ActionIdIterator & SequenceMask);
	
	// ActionId of 0 is invalid, when looping around increment one more time
	if (ActionIdIterator == 0u)
	{
		ActionIdIterator++;
	}

	if (IsServer())
	{
		// Set the last 16th bit as 1 for ActionId created by the server
		return uint16(ActionIdIterator | ServerBit);
	}
	else // Client
	{
		return ActionIdIterator;
	}
}

FAsyncNetworkPhysicsComponentOutput& FAsyncNetworkPhysicsComponent::GetAsyncOutput_Internal()
{
	FAsyncNetworkPhysicsComponentOutput& AsyncOutput = GetProducerOutputData_Internal();

	// InputData marshal from PT to GT is needed for: LocallyControlled and Server
	if ((IsLocallyControlled() || IsServer()) && AsyncOutput.InputData == nullptr && InputHistory != nullptr)
	{
		AsyncOutput.InputData = InputHistory->CreateNew();
	}

	// StateData marshal from PT to GT is needed for: Server
	if (IsServer() && AsyncOutput.StateData == nullptr && StateHistory != nullptr)
	{
		AsyncOutput.StateData = StateHistory->CreateNew();
		AsyncOutput.StateData->ResizeDataHistory(StatesToNetwork);
	}

	return AsyncOutput;
}

void FAsyncNetworkPhysicsComponent::OnAddResimulationRequest_Internal(const int32 PhysicsStep)
{
	if(Chaos::RewindBeforeAdvance != 0)
	{
		ConsumeAsyncInput_Internal(PhysicsStep, true);
	}
}

void FAsyncNetworkPhysicsComponent::OnPreProcessInputs_Internal(const int32 PhysicsStep)
{
	QUICK_SCOPE_CYCLE_COUNTER(AsyncNetworkPhysicsComponent_OnPreProcessInputs_Internal);

	bool bIsFirstResimFrame = false;
	bool bIsSolverResim = false;
	if (Chaos::FPBDRigidsEvolution* Evolution = GetEvolution())
	{
		bIsSolverResim = Evolution->IsResimming();
		bIsFirstResimFrame = Evolution->IsResetting();
	}

	// Reset per-entry resim-claim flags at the start of a resim pass.
	if (bIsFirstResimFrame)
	{
		if (PendingProposedActions.IsInitialized())
		{
			PendingProposedActions.ClearResimClaims();
		}
		if (PredictedActions.IsInitialized())
		{
			PredictedActions.ClearResimClaims();
		}
		if (ConfirmedActions.IsInitialized())
		{
			ConfirmedActions.ClearResimClaims();
		}
	}

	if (!bIsSolverResim)
	{
		ConsumeAsyncInput_Internal(PhysicsStep, Chaos::RewindBeforeAdvance == 0);
	}

	const FNetworkPhysicsSettingsNetworkPhysicsComponent& ComponentSettings = GetComponentSettings();
	const bool bIsServer = IsServer();

#if DEBUG_NETWORK_PHYSICS
	{
		const int32 InputBufferSize = (bIsServer && InputHistory) ? (InputHistory->GetLatestFrame() - PhysicsStep) : 0;
		const FString NetRoleString = bIsServer ? FString("SERVER") : (IsLocallyControlled() ? FString("AUTONO") : FString("PROXY "));
		UE_LOGF(LogChaos, Log, "%ls | PT | OnPreProcessInputsInternal | At Frame %d | IsResim: %d | FirstResimFrame: %d | InputBuffer: %d | Name = %ls", *NetRoleString, PhysicsStep, bIsSolverResim, bIsFirstResimFrame, InputBufferSize, *GetActorName());
	}
#endif

	{
		Chaos::FReadPhysicsObjectInterface_Internal Interface = Chaos::FPhysicsObjectInternalInterface::GetRead();
		
		bool bCanApplyData = true;
		if (bIsSolverResim)
		{
			if (Chaos::FPBDRigidParticleHandle* Particle = Interface.GetRigidParticle(PhysicsObject))
			{
				bCanApplyData = (Particle->ResimType() != Chaos::EResimType::FrozenDuringResim);
			}
		}
		
		// Apply replicated state on clients if we are resimulating or on simulated proxies if setting is enabled
		const bool bApplySimProxyState = ComponentSettings.GetApplySimProxyStateAtRuntime() && !bIsServer && !IsLocallyControlled();
		
		if ((bApplySimProxyState || bIsSolverResim) && StateHistory && StateData && bCanApplyData)
		{
			FNetworkPhysicsPayload* PhysicsData = StateData.Get();
			PhysicsData->LocalFrame = PhysicsStep;

			/** Extract state to apply, only apply states that are received from the server (not predicted states)
			* Require exact frame unless bAllowRewindToClosestState is true, then allow non-exact frame on first resim frame */ 
			const bool bExactFrame = PhysicsReplicationCVars::ResimulationCVars::bAllowRewindToClosestState ? !bIsFirstResimFrame : true;
			if (StateHistory->ExtractData(PhysicsStep, bIsFirstResimFrame, PhysicsData, (bExactFrame && bIsSolverResim)) && PhysicsData->bReceivedData)
			{
				if (bIsUsingLegacyData)
				{
					if (ActorComponent.IsValid() && ActorComponent.Get()->IsBeingDestroyed() == false)
					{
						FNetworkPhysicsData* LegacyData = static_cast<FNetworkPhysicsData*>(PhysicsData);
						LegacyData->ApplyData(ActorComponent.Get());
					}
				}
				else
				{
					if (ImplementationInterface_Internal)
					{
						ImplementationInterface_Internal->ApplyState(*PhysicsData);
					}
				}
#if DEBUG_NETWORK_PHYSICS
				UE_LOGF(LogChaos, Log, "			Applying extracted state from history | bExactFrame = %d | LocalFrame = %d | ServerFrame = %d | bDataAltered = %d | Data: %ls"
					, bExactFrame, PhysicsData->LocalFrame, PhysicsData->ServerFrame, PhysicsData->bDataAltered, *PhysicsData->DebugData());
#endif
			}
#if DEBUG_NETWORK_PHYSICS
			else if (PhysicsStep <= StateHistory->GetLatestFrame())
			{
				UE_LOGF(LogChaos, Log, "		Non-Determinism: FAILED to extract and apply state from history | bExactFrame = %d | -- Printing history --", bExactFrame);
				StateHistory->DebugData(FString::Printf(TEXT("StateHistory | Component = %s"), *GetActorName()));
			}
#endif
		}

		// Apply replicated inputs on server and simulated proxies if setting is enabled, and on local player if we are resimulating
		const bool bApplyServerInput = bIsServer && !IsLocallyControlled();
		const bool bApplySimProxyInputAtRuntime = !bIsServer && !IsLocallyControlled() && !bIsSolverResim && ComponentSettings.GetApplySimProxyInputAtRuntime();
		const bool bIsReplicatingWithResim = GetPhysicsReplicationMode() == EPhysicsReplicationMode::Resimulation;

		// Cache LastInputFrameApplied before resim for sim-proxies so we can restore it after the resim since it's used during normal simulation also for sim-proxies
		if (!IsLocallyControlled() && !IsServer())
		{
			if (bIsSolverResim && bIsFirstResimFrame && !SimProxyCachedLastInputFrame.IsSet())
			{
				SimProxyCachedLastInputFrame = LastInputFrameApplied;
			}
			else if (!bIsSolverResim && SimProxyCachedLastInputFrame.IsSet())
			{
				LastInputFrameApplied = SimProxyCachedLastInputFrame.GetValue();
				SimProxyCachedLastInputFrame.Reset();
			}
		}

		if ((bApplyServerInput || bApplySimProxyInputAtRuntime || bIsSolverResim) && InputHistory && InputData && bCanApplyData)
		{
			FNetworkPhysicsPayload* PhysicsData = InputData.Get();

			// First resim frame: LastInputFrameApplied is stale from the pre-resim timeline so use PhysicsStep directly.
			// Subsequent frames: use LastInputFrameApplied + 1 so that any gap from coalesced/skipped resim frames
			// is detected and their inputs are merged via MergeData into the frame that actually simulates.
			int32 NextExpectedInputFrame = (bIsSolverResim && bIsFirstResimFrame) ? PhysicsStep : LastInputFrameApplied + 1;

			// There are important inputs earlier than upcoming input to apply
			if (!bIsSolverResim && NewImportantInputFrame < NextExpectedInputFrame)
			{
				if (ComponentSettings.GetApplyDataInsteadOfMergeData())
				{
#if DEBUG_NETWORK_PHYSICS
					UE_LOGF(LogChaos, Log, "		Non-Determinism: Reapplying multiple data due to receiving an important data that was previously missed. FromFrame: %d | ToFrame: %d | IsLocallyControlled = %d", NewImportantInputFrame, (NextExpectedInputFrame - 1), IsLocallyControlled());
#endif
					if (bIsUsingLegacyData)
					{
						if (ActorComponent.IsValid() && ActorComponent.Get()->IsBeingDestroyed() == false)
						{
							// Apply all inputs in range
							InputHistory->ApplyDataRange(NewImportantInputFrame, NextExpectedInputFrame - 1, ActorComponent.Get(), /*bOnlyImportant*/false);
						}
					}
/* // Importance is not implemented into the new network flow (yet?)
					else
					{
						if (InputHelper && ImplementationInterface_Internal)
						{
							InputHelper->ApplyDataRange(InputHistory.Get(), NewImportantInputFrame, NextExpectedInputFrame - 1, *ImplementationInterface_Internal);
						}
					}
*/
				}
				else
				{
					// Merge all inputs from earliest new important
					NextExpectedInputFrame = NewImportantInputFrame;
#if DEBUG_NETWORK_PHYSICS
					UE_LOGF(LogChaos, Log, "		Non-Determinism: Prepare to reapply multiple data through MergeData due to receiving an important data that was previously missed. FromFrame: %d | ToFrame: %d | IsLocallyControlled = %d", NewImportantInputFrame, (NextExpectedInputFrame - 1), IsLocallyControlled());
#endif
				}
			}

			int32 InputFrameToApply = PhysicsStep;

			// For sim-proxies applying inputs at runtime
			if (bApplySimProxyInputAtRuntime)
			{
				// Get either the latest received input frame
				if (InputHistory->GetLatestFrame() > 0)
				{
					InputFrameToApply = FMath::Min(InputFrameToApply, InputHistory->GetLatestFrame());
				}

				// Or if the sim-proxy should run in the interpolated timeline, get the last received state frame
				if (IsNonResimSimProxy())
				{
					const int32 LatestStateFrame = GetLatestReceivedStateFrame();
					if (LatestStateFrame > 0)
					{
						InputFrameToApply = FMath::Min(InputFrameToApply, LatestStateFrame);
					}
				}
			}

			const bool RequireExactFrame = bIsServer && !ComponentSettings.GetAllowInputExtrapolation();
			if (InputHistory->ExtractData(InputFrameToApply, bIsFirstResimFrame, PhysicsData, RequireExactFrame))
			{
				const int32 CurrentInputFrame = PhysicsData->LocalFrame;

				// Check if the extracted data was altered and if we have a hole in the buffer
				if (bIsServer && PhysicsData->bDataAltered)
				{
					if (PhysicsStep < InputHistory->GetLatestFrame())
					{
						// A missing input was detected and buffer is not empty, inform the owning client to send more inputs in each RPC to not get a gaps in the buffer
						// NOTE: We don't send more extra inputs when the buffer runs empty since that case is corrected via time dilation, not sending extra inputs
						MissingInputCount++;
					}
#if DEBUG_NETWORK_PHYSICS
					else
					{
						UE_LOGF(LogChaos, Log, "		Non-Determinism: Input buffer Empty, input for frame %d was extrapolated", PhysicsStep);
					}
#endif
				}

				// Merge/apply inputs from frames between the last applied input and the current one.
				// Handles sim-proxy frame gaps at runtime and coalesced/skipped frames during resim.
				if (NextExpectedInputFrame > 0 && CurrentInputFrame > NextExpectedInputFrame)
				{
					if (!bIsSolverResim && ComponentSettings.GetApplyDataInsteadOfMergeData())
					{
#if DEBUG_NETWORK_PHYSICS
						UE_LOGF(LogChaos, Log, "		Non-Determinism: Applying multiple data instead of merging, from LocalFrame %d into LocalFrame %d | IsLocallyControlled = %d", NextExpectedInputFrame, PhysicsData->LocalFrame, IsLocallyControlled());
#endif
						// Iterate over each input and call ApplyData, except on the last, it will get handled by the normal ApplyData call further down
						for (; NextExpectedInputFrame <= CurrentInputFrame; NextExpectedInputFrame++)
						{
							/** Note, extract the last frame but don't apply it here, it will be applied later in the flow
							* Also note that we allow extracting non-exact data on the last frame, unless required by settings, to get the same data extracted as we did in the original ExtractData above */
							bool bLastInput = NextExpectedInputFrame == CurrentInputFrame;
							if (InputHistory->ExtractData(NextExpectedInputFrame, bIsFirstResimFrame, PhysicsData, (RequireExactFrame || !bLastInput)) && !bLastInput)
							{
								// Perform input decay on sim-proxy at runtime if replicated with resimulation
								if (PhysicsReplicationCVars::ResimulationCVars::bEnableInputDecay && !IsNonResimSimProxy() && bApplySimProxyInputAtRuntime && ComponentSettings.GetApplyInputDecaySimProxyInputAtRuntime())
								{
									const float RuntimeInputDecay = GetCurrentSimProxyInputDecayAtRuntime();
									PhysicsData->DecayData(RuntimeInputDecay);
								}

								if (bIsUsingLegacyData)
								{
									if (ActorComponent.IsValid() && ActorComponent.Get()->IsBeingDestroyed() == false)
									{
										FNetworkPhysicsData* LegacyData = static_cast<FNetworkPhysicsData*>(PhysicsData);
										LegacyData->ApplyData(ActorComponent.Get());
									}
								}
								else
								{
									if (ImplementationInterface_Internal)
									{
										ImplementationInterface_Internal->ApplyInput(*PhysicsData);
									}
								}
							}
						}
					}
					else 
					{
						// Merge skipped inputs into the target frame (resim coalescing and sim-proxy runtime gaps)
#if DEBUG_NETWORK_PHYSICS
						UE_LOGF(LogChaos, Log, "		Non-Determinism: Merging inputs from LocalFrame %d into LocalFrame %d | IsLocallyControlled = %d | IsResim = %d", NextExpectedInputFrame, PhysicsData->LocalFrame, IsLocallyControlled(), bIsSolverResim);
#endif
						InputHistory->MergeData(NextExpectedInputFrame, PhysicsData);
					}
				}

				// Apply input decay during resim when inputs are stale (after merge so decay covers the combined input)
				if (bIsSolverResim && PhysicsReplicationCVars::ResimulationCVars::bEnableInputDecay && InputHistory->GetLatestFrame() < PhysicsStep)
				{
					const float InputDecay = GetCurrentInputDecay();
					PhysicsData->DecayData(InputDecay);
				}

				// If the extracted input data was altered (merged, extrapolated, interpolated, injected) on the server, record it into the history for it to get replicated to clients
				if (bIsServer && PhysicsData->bDataAltered)
				{
					// Explicitly say this input was not received, since it was altered by the server and when receiving the input for this frame it should get processed as altered but not received when calling ReceiveNewData()
					PhysicsData->bReceivedData = false;
					PhysicsData->bImportant = false;
					InputHistory->RecordData(InputFrameToApply, PhysicsData);
				}

				// Perform input decay on sim-proxy at runtime if replicated with resimulation
				if (PhysicsReplicationCVars::ResimulationCVars::bEnableInputDecay && !bIsSolverResim && bIsReplicatingWithResim && bApplySimProxyInputAtRuntime && ComponentSettings.GetApplyInputDecaySimProxyInputAtRuntime())
				{
					const float RuntimeInputDecay = GetCurrentSimProxyInputDecayAtRuntime();
					PhysicsData->DecayData(RuntimeInputDecay);
				}

				if (bIsUsingLegacyData)
				{
					if (ActorComponent.IsValid() && ActorComponent.Get()->IsBeingDestroyed() == false)
					{
						FNetworkPhysicsData* LegacyData = static_cast<FNetworkPhysicsData*>(PhysicsData);
						LegacyData->ApplyData(ActorComponent.Get());
					}
				}
				else
				{
					if (ImplementationInterface_Internal)
					{
						ImplementationInterface_Internal->ApplyInput(*PhysicsData);
					}
				}

				// Track for all roles so adaptive coalescing can detect skipped frames and merge their inputs
				LastInputFrameApplied = PhysicsData->LocalFrame;

#if DEBUG_NETWORK_PHYSICS
				{
					UE_LOGF(LogChaos, Log, "			Applying extracted input from history | LocalFrame = %d | ServerFrame = %d | bDataAltered = %d | IsResim = %d | IsLocallyControlled = %d | InputDecay = %f | Data: %ls"
						, PhysicsData->LocalFrame, PhysicsData->ServerFrame, PhysicsData->bDataAltered, bIsSolverResim, IsLocallyControlled(), GetCurrentInputDecay(), *PhysicsData->DebugData());
				}
#endif
			}
#if DEBUG_NETWORK_PHYSICS
			else if (InputFrameToApply <= InputHistory->GetLatestFrame())
			{
				UE_LOGF(LogChaos, Log, "		Non-Determinism: FAILED to extract and apply input from history | IsResim = %d | IsLocallyControlled = %d | -- Printing history --", bIsSolverResim, IsLocallyControlled());
				InputHistory->DebugData(FString::Printf(TEXT("InputHistory | Name = %s"), *GetActorName()));
			}
#endif
		}
	}

	 // Apply Actions
	ApplyActionsForFrame(PhysicsStep);

	NewImportantInputFrame = INT_MAX;
}

void FAsyncNetworkPhysicsComponent::OnPostProcessInputs_Internal(const int32 PhysicsStep)
{
	QUICK_SCOPE_CYCLE_COUNTER(AsyncNetworkPhysicsComponent_OnPostProcessInputs_Internal);

	const FNetworkPhysicsSettingsNetworkPhysicsComponent& ComponentSettings = GetComponentSettings();
	const bool bIsServer = IsServer();

	bool bIsFirstResimFrame = false;
	bool bIsSolverResim = false;
	if (Chaos::FPBDRigidsEvolution* Evolution = GetEvolution())
	{
		bIsSolverResim = Evolution->IsResimming();
		bIsFirstResimFrame = Evolution->IsResetting();
	}

#if DEBUG_NETWORK_PHYSICS
	{
		FString NetRoleString = bIsServer ? FString("SERVER") : (IsLocallyControlled() ? FString("AUTONO") : FString("PROXY "));
		UE_LOGF(LogChaos, Log, "%ls | PT | OnPostProcessInputsInternal | At Frame %d | IsResim: %d | FirstResimFrame: %d | Name = %ls", *NetRoleString, PhysicsStep, bIsSolverResim, bIsFirstResimFrame, *GetActorName());
	}
#endif

	// Cache current input if we are locally controlled
	const bool bShouldCacheInputHistory = IsLocallyControlled() && !bIsSolverResim;
	if (bShouldCacheInputHistory && (InputData != nullptr))
	{
		// Prepare to gather input data
		FNetworkPhysicsPayload* PhysicsData = InputData.Get();
		PhysicsData->PrepareFrame(PhysicsStep, bIsServer, UE::NetworkPhysicsUtils::GetNetworkPhysicsTickOffset_Internal(GetSolver()));

		if (bIsUsingLegacyData)
		{
			if (ActorComponent.IsValid() && ActorComponent.Get()->IsBeingDestroyed() == false)
			{
				// Gather input data from implementation
				FNetworkPhysicsData* LegacyData = static_cast<FNetworkPhysicsData*>(PhysicsData);
				LegacyData->BuildData(ActorComponent.Get());
			}
		}
		else
		{
			if (ImplementationInterface_Internal)
			{
				// Gather input data from implementation
				ImplementationInterface_Internal->BuildInput(*PhysicsData);
			}
		}

		// Record input in history
		InputHistory->RecordData(PhysicsStep, PhysicsData);

#if DEBUG_NETWORK_PHYSICS
		{
			UE_LOGF(LogChaos, Log, "		Recording input into history | LocalFrame = %d | ServerFrame = %d | bDataAltered = %d | Input: %ls "
				, PhysicsData->LocalFrame, PhysicsData->ServerFrame, PhysicsData->bDataAltered, *PhysicsData->DebugData());
		}
#endif
	}

	if (PhysicsReplicationCVars::ResimulationCVars::bRecordStatePostSolve == false)
	{
		// Cache current state if this is the server or we are comparing predicted states clients
		const bool bShouldCacheStateHistory = bIsServer
			|| (ComponentSettings.GetCompareStateToTriggerRewind(bCompareStateToTriggerRewind) && (IsLocallyControlled() || ComponentSettings.GetCompareStateToTriggerRewindIncludeSimProxies(bCompareStateToTriggerRewindIncludeSimProxies)));

		if (StateHistory && StateData && bShouldCacheStateHistory)
		{
			// Prepare to gather state data
			FNetworkPhysicsPayload* PhysicsData = StateData.Get();
			PhysicsData->PrepareFrame(PhysicsStep, bIsServer, UE::NetworkPhysicsUtils::GetNetworkPhysicsTickOffset_Internal(GetSolver()));

			if (bIsUsingLegacyData)
			{
				if (ActorComponent.IsValid() && ActorComponent.Get()->IsBeingDestroyed() == false)
				{
					// Gather state data from implementation
					FNetworkPhysicsData* LegacyData = static_cast<FNetworkPhysicsData*>(PhysicsData);
					LegacyData->BuildData(ActorComponent.Get());
				}
			}
			else
			{
				if (ImplementationInterface_Internal)
				{
					// Gather state data from implementation
					ImplementationInterface_Internal->BuildState(*PhysicsData);
				}
			}

			// Record state in history
			StateHistory->RecordData(PhysicsStep, PhysicsData);

	#if DEBUG_NETWORK_PHYSICS
			{
				UE_LOGF(LogChaos, Log, "		Recording state into history | LocalFrame = %d | ServerFrame = %d | bDataAltered = %d | State: %ls "
					, PhysicsData->LocalFrame, PhysicsData->ServerFrame, PhysicsData->bDataAltered, *PhysicsData->DebugData());
			}
	#endif
		}
	}

	if (bIsSolverResim == false && PhysicsReplicationCVars::ResimulationCVars::bRecordStatePostSolve == false)
	{
		const bool bHasInputOrState = (InputHistory != nullptr) || (StateHistory != nullptr);
		const bool bHasActions = ConfirmedActions.IsInitialized() || PendingProposedActions.IsInitialized();

		if (bHasInputOrState || bHasActions)
		{
			FAsyncNetworkPhysicsComponentOutput& AsyncOutput = GetAsyncOutput_Internal();

			if (bHasInputOrState)
			{
				SendInputData_Internal(AsyncOutput, PhysicsStep);
				SendStateData_Internal(AsyncOutput, PhysicsStep);
			}
			if (bHasActions)
			{
				SendActionData_Internal(AsyncOutput, PhysicsStep);
			}

			FinalizeOutputData_Internal();
		}
	}
}

void FAsyncNetworkPhysicsComponent::OnPreSimulate_Internal()
{
	using namespace PhysicsReplicationCVars::ResimulationCVars;

	if (bSimDecayNetPhysicsCompEnable)
	{
		if (IsServer() || IsLocallyControlled() || !PhysicsObject)
		{
			return;
		}

		Chaos::FPBDRigidsSolver* RigidsSolver = GetRigidSolver();
		if (!RigidsSolver || !RigidsSolver->GetEvolution() || RigidsSolver->GetEvolution()->IsResimming())
		{
			return;
		}

		if (!SimDecaySettings.IsValid())
		{
			// Cache local weak pointer to the simulation decay settings for this particle
			if (IPhysicsReplicationAsync* PhysRep = RigidsSolver->GetPhysicsReplication_Internal())
			{
				SimDecaySettings = PhysRep->FindOrAddParticleSimDecaySettings(PhysicsObject);
			}

			if (!SimDecaySettings.IsValid())
			{
				return;
			}

			SimDecayRegisteredHandle = PhysicsObject;
		}

		// Update the simulation decay settings in PhysicsReplication
		TSharedPtr<FParticleSimDecaySettings> SimDecaySettingsPinned = SimDecaySettings.Pin();
		SimDecaySettingsPinned->bApplyDecayAtRuntime = bSimDecayNetPhysicsCompApplyAtRuntime;
		SimDecaySettingsPinned->InputPredictionFramesAverage = InputPredictionFramesAverage;
		SimDecaySettingsPinned->StaticTimeScale = SimDecayNetPhysicsCompStaticTimeScale;
		SimDecaySettingsPinned->SetDynamicSettings(bSimDecayNetPhysicsCompUseDynamic, SimDecayNetPhysicsCompDynamicBase, SimDecayNetPhysicsCompDynamicMin, SimDecayNetPhysicsCompDynamicMax);
	}
	else if (SimDecaySettings.IsValid())
	{
		if (Chaos::FPBDRigidsSolver* RigidsSolver = GetRigidSolver())
		{
			if (IPhysicsReplicationAsync * PhysRep = RigidsSolver->GetPhysicsReplication_Internal())
			{
				PhysRep->RemoveParticleSimDecaySettings(SimDecayRegisteredHandle);
			}
		}

		SimDecaySettings.Reset();
		SimDecayRegisteredHandle = nullptr;
	}
}

void FAsyncNetworkPhysicsComponent::OnPostSolve_Internal()
{
	Chaos::FPBDRigidsSolver* RigidsSolver = GetRigidSolver();
	if (!RigidsSolver || !RigidsSolver->GetEvolution())
	{
		return;
	}
	const bool bIsSolverResim = RigidsSolver->GetEvolution()->IsResimming();
	const int32 PhysicsStep = RigidsSolver->GetCurrentFrame();
	const int32 NextPhysicsStep = PhysicsStep + 1;

	const FNetworkPhysicsSettingsNetworkPhysicsComponent& ComponentSettings = GetComponentSettings();
	const bool bIsServer = IsServer();

	if (PhysicsReplicationCVars::ResimulationCVars::bRecordStatePostSolve == true)
	{
		// Cache current state if this is the server or we are comparing predicted states clients
		const bool bShouldCacheStateHistory = bIsServer
			|| (ComponentSettings.GetCompareStateToTriggerRewind(bCompareStateToTriggerRewind) && (IsLocallyControlled() || ComponentSettings.GetCompareStateToTriggerRewindIncludeSimProxies(bCompareStateToTriggerRewindIncludeSimProxies)));

		if (StateHistory && StateData && bShouldCacheStateHistory)
		{
			// Prepare to gather state data
			FNetworkPhysicsPayload* PhysicsData = StateData.Get();
			PhysicsData->PrepareFrame(NextPhysicsStep, bIsServer, UE::NetworkPhysicsUtils::GetNetworkPhysicsTickOffset_Internal(GetSolver()));

			if (bIsUsingLegacyData)
			{
				if (ActorComponent.IsValid() && ActorComponent.Get()->IsBeingDestroyed() == false)
				{
					// Gather state data from implementation
					FNetworkPhysicsData* LegacyData = static_cast<FNetworkPhysicsData*>(PhysicsData);
					LegacyData->BuildData(ActorComponent.Get());
				}
			}
			else
			{
				if (ImplementationInterface_Internal)
				{
					// Gather state data from implementation
					ImplementationInterface_Internal->BuildState(*PhysicsData);
				}
			}

			// Record state in history
			StateHistory->RecordData(NextPhysicsStep, PhysicsData);

	#if DEBUG_NETWORK_PHYSICS
			{
				UE_LOGF(LogChaos, Log, "		Recording state into history | LocalFrame = %d | ServerFrame = %d | bDataAltered = %d | State: %ls "
					, PhysicsData->LocalFrame, PhysicsData->ServerFrame, PhysicsData->bDataAltered, *PhysicsData->DebugData());
			}
	#endif
		}
	}

	if (bIsSolverResim == false && PhysicsReplicationCVars::ResimulationCVars::bRecordStatePostSolve == true)
	{
		const bool bHasInputOrState = (InputHistory != nullptr) || (StateHistory != nullptr);
		const bool bHasActions = ConfirmedActions.IsInitialized() || PendingProposedActions.IsInitialized();

		if (bHasInputOrState || bHasActions)
		{
			FAsyncNetworkPhysicsComponentOutput& AsyncOutput = GetAsyncOutput_Internal();

			if (bHasInputOrState)
			{
				SendInputData_Internal(AsyncOutput, PhysicsStep);
				SendStateData_Internal(AsyncOutput, NextPhysicsStep); // We cache state at the end of the solve which is the state that next frame starts with
			}
			if (bHasActions)
			{
				SendActionData_Internal(AsyncOutput, PhysicsStep);
			}

			FinalizeOutputData_Internal();
		}
	}
}

void FAsyncNetworkPhysicsComponent::SendInputData_Internal(FAsyncNetworkPhysicsComponentOutput& AsyncOutput, const int32 PhysicsStep)
{
	const bool bIsServer = IsServer();

	// Inputs are sent from the server or locally controlled actors/pawns
	if (AsyncOutput.InputData && InputHistory && (IsLocallyControlled() || bIsServer))
	{
		const FNetworkPhysicsSettingsNetworkPhysicsComponent& ComponentSettings = GetComponentSettings();

		if (bIsServer)
		{
			UpdateDynamicInputReplicationScaling();
			AsyncOutput.InputsToNetwork_Owner = InputsToNetwork_Owner;

			if (ComponentSettings.GetDynamicInputBufferScalingEnabled())
			{
				AsyncOutput.TargetBufferSizeMs = UpdateDynamicInputBufferScaling();
			}
		}

		// Send latest N frames from history
		const int32 ToFrame = FMath::Max(0, PhysicsStep);

		// -- Default / Unreliable Flow --
		if (ComponentSettings.GetEnableUnreliableFlow())
		{
			const uint16 NumInputsToNetwork = bIsServer ? InputsToNetwork_Simulated : InputsToNetwork_Owner;
			const int32 FromFrame = FMath::Max(0, ToFrame - NumInputsToNetwork - 1); // Remove 1 since both ToFrame and FromFrame are inclusive

			AsyncOutput.InputData->ResizeDataHistory(NumInputsToNetwork);

			if (InputHistory->CopyData(*AsyncOutput.InputData, FromFrame, ToFrame, /*bIncludeUnimportant*/ true, /*bIncludeImportant*/ ComponentSettings.GetEnableReliableFlow() == false))
			{
#if DEBUG_NETWORK_PHYSICS
				{
					const int32 LocalFrame = GetRigidSolver()->GetCurrentFrame();
					const int32 ServerFrame = bIsServer ? LocalFrame : LocalFrame + UE::NetworkPhysicsUtils::GetNetworkPhysicsTickOffset_Internal(GetSolver());
					FString NetRoleString = bIsServer ? FString("SERVER") : (IsLocallyControlled() ? FString("AUTONO") : FString("PROXY "));
					AsyncOutput.InputData->DebugData(FString::Printf(TEXT("%s | PT | SendInputData_Internal | UNRELIABLE | CurrentLocalFrame = %d | CurrentServerFrame = %d | Name: %s"), *NetRoleString, LocalFrame, ServerFrame, *GetActorName()));
				}
#endif
			}
		}

		// -- Important / Reliable flow --
		if (ComponentSettings.GetEnableReliableFlow())
		{
			/* Get the latest valid frame that can hold new important data:
			* 1. Frame after last time we called SendInputData
			* 2. Earliest possible frame in history */
			const int32 FromFrame = FMath::Max(LastInputSendFrame + 1, ToFrame - InputHistory->GetHistorySize());

			// Check if we have important data to marshal
			const int32 Count = InputHistory->CountValidData(FromFrame, ToFrame, /*bIncludeUnimportant*/ false, /*bIncludeImportant*/ true);
			if (Count > 0)
			{
				// Create new data collection for marshaling
				const int32 Idx = AsyncOutput.InputDataImportant.Add(InputHistory->CreateNew());
				AsyncOutput.InputDataImportant[Idx]->ResizeDataHistory(Count);

				// Copy over data
				if (InputHistory->CopyData(*AsyncOutput.InputDataImportant[Idx], FromFrame, ToFrame, /*bIncludeUnimportant*/ false, /*bIncludeImportant*/ true))
				{
#if DEBUG_NETWORK_PHYSICS
					{
						const int32 LocalFrame = GetRigidSolver()->GetCurrentFrame();
						const int32 ServerFrame = bIsServer ? LocalFrame : LocalFrame + UE::NetworkPhysicsUtils::GetNetworkPhysicsTickOffset_Internal(GetSolver());
						FString NetRoleString = bIsServer ? FString("SERVER") : (IsLocallyControlled() ? FString("AUTONO") : FString("PROXY "));
						AsyncOutput.InputDataImportant[Idx]->DebugData(FString::Printf(TEXT("%s | PT | SendInputData_Internal | RELIABLE | CurrentLocalFrame = %d | CurrentServerFrame = %d | Name: %s"), *NetRoleString, LocalFrame, ServerFrame, *GetActorName()));
					}
#endif
				}
				
			}
		}
		LastInputSendFrame = InputHistory->GetLatestFrame();
	}
}

void FAsyncNetworkPhysicsComponent::SendStateData_Internal(FAsyncNetworkPhysicsComponentOutput& AsyncOutput, const int32 PhysicsStep)
{
	if (IsServer() && StateHistory && AsyncOutput.StateData)
	{
		const FNetworkPhysicsSettingsNetworkPhysicsComponent& ComponentSettings = GetComponentSettings();

		// Send latest N frames from history
		const int32 ToFrame = FMath::Max(0, PhysicsStep);

		// -- Default / Unreliable Flow --
		if (ComponentSettings.GetEnableUnreliableFlow())
		{
			const int32 FromFrame = FMath::Max(0, ToFrame - StatesToNetwork - 1); // Remove 1 since both ToFrame and FromFrame are inclusive

			// Resize marshaling history if needed
			AsyncOutput.StateData->ResizeDataHistory(StatesToNetwork);

			if (StateHistory->CopyData(*AsyncOutput.StateData, FromFrame, ToFrame, /*bIncludeUnimportant*/ true, /*bIncludeImportant*/ ComponentSettings.GetEnableReliableFlow() == false))
			{
#if DEBUG_NETWORK_PHYSICS
				{
					const int32 LocalFrame = GetRigidSolver()->GetCurrentFrame();
					const int32 ServerFrame = IsServer() ? LocalFrame : LocalFrame + UE::NetworkPhysicsUtils::GetNetworkPhysicsTickOffset_Internal(GetSolver());
					AsyncOutput.StateData->DebugData(FString::Printf(TEXT("SERVER | PT | SendStateData_Internal | UNRELIABLE | CurrentLocalFrame = %d | CurrentServerFrame = %d | Name: %s"), LocalFrame, ServerFrame, *GetActorName()));
				}
#endif
			}
		}

		// -- Important / Reliable flow --
		if (ComponentSettings.GetEnableReliableFlow())
		{
			/* Get the latest valid frame that can hold new important data:
			* 1. Frame after last time we called SendStateData
			* 2. Earliest possible frame in history */
			const int32 FromFrame = FMath::Max(LastStateSendFrame + 1, ToFrame - StateHistory->GetHistorySize());

			// Check if we have important data to marshal
			const int32 Count = StateHistory->CountValidData(FromFrame, ToFrame, /*bIncludeUnimportant*/ false, /*bIncludeImportant*/ true);
			if (Count > 0)
			{
				// Create new data collection for marshaling
				const int32 Idx = AsyncOutput.StateDataImportant.Add(StateHistory->CreateNew());
				AsyncOutput.StateDataImportant[Idx]->ResizeDataHistory(Count);

				// Copy over data
				if (StateHistory->CopyData(*AsyncOutput.StateDataImportant[Idx], FromFrame, ToFrame, /*bIncludeUnimportant*/ false, /*bIncludeImportant*/ true))
				{
#if DEBUG_NETWORK_PHYSICS
					{
						const int32 LocalFrame = GetRigidSolver()->GetCurrentFrame();
						const int32 ServerFrame = IsServer() ? LocalFrame : LocalFrame + UE::NetworkPhysicsUtils::GetNetworkPhysicsTickOffset_Internal(GetSolver());
						AsyncOutput.StateDataImportant[Idx]->DebugData(FString::Printf(TEXT("SERVER | PT | SendStateData_Internal | RELIABLE | CurrentLocalFrame = %d | CurrentServerFrame = %d | Name: %s"), LocalFrame, ServerFrame, *GetActorName()));
					}
#endif
				}
			}
		}
		LastStateSendFrame = StateHistory->GetLatestFrame();
	}
}

void FAsyncNetworkPhysicsComponent::SendActionData_Internal(FAsyncNetworkPhysicsComponentOutput& AsyncOutput, const int32 PhysicsStep)
{
	if (IsServer())
	{
		ConfirmedActions.GetActionsToReplicate(AsyncOutput.UnreliableActionData, AsyncOutput.ReliableActionData);
	}
	else if (IsLocallyControlled()) // Autonomous Proxy Client
	{
		PendingProposedActions.GetActionsToReplicate(AsyncOutput.UnreliableActionData, AsyncOutput.ReliableActionData);
	}
}

Chaos::FPBDRigidsSolver* FAsyncNetworkPhysicsComponent::GetRigidSolver()
{
	return static_cast<Chaos::FPBDRigidsSolver*>(GetSolver());
}

Chaos::FPBDRigidsEvolution* FAsyncNetworkPhysicsComponent::GetEvolution()
{
	if (Chaos::FPBDRigidsSolver* RigidSolver = GetRigidSolver())
	{
		return RigidSolver->GetEvolution();
	}
	return nullptr;
}

void FAsyncNetworkPhysicsComponent::TriggerResimulation(int32 ResimFrame)
{
	if (ResimFrame != INDEX_NONE)
	{
		if (Chaos::FPBDRigidsSolver* RigidSolver = GetRigidSolver())
		{
			if (Chaos::FRewindData* RewindData = RigidSolver->GetRewindData())
			{
				Chaos::FReadPhysicsObjectInterface_Internal Interface = Chaos::FPhysicsObjectInternalInterface::GetRead();
				Chaos::FPBDRigidParticleHandle* Particle = Interface.GetRigidParticle(PhysicsObject);

				// Set resim frame in rewind data
				RewindData->RequestResimulation(ResimFrame, Particle);
			}
		}
	}
}

void FAsyncNetworkPhysicsComponent::OnRewindDataResize_Internal(int32 InNumFrames)
{
	if (InputHistory)
	{
		InputHistory->ResizeDataHistory(InNumFrames);
	}
	if (StateHistory)
	{
		StateHistory->ResizeDataHistory(InNumFrames);
	}
}

const float FAsyncNetworkPhysicsComponent::GetCurrentInputDecay() const
{
	if (!InputHistory)
	{
		return 0.0f;
	}

	const Chaos::FPhysicsSolverBase* BaseSolver = GetSolver();
	if (!BaseSolver || !BaseSolver->IsResimming())
	{
		return 0.0f;
	}

	const Chaos::FRewindData* RewindData = BaseSolver->GetRewindData();
	if (!RewindData)
	{
		return 0.0f;
	}

	const int32 LatestReceivedFrame = InputHistory->GetLatestFrame();

	if (RewindData->CurrentFrame() <= LatestReceivedFrame)
	{
		return 0.0f;
	}

	const FNetworkPhysicsSettingsNetworkPhysicsComponent& ComponentSettings = GetComponentSettings();
	const FRuntimeFloatCurve& InputDecayCurve = ComponentSettings.GetInputDecayCurve();

	const float NumPredictedInputs = RewindData->CurrentFrame() - LatestReceivedFrame; // Number of frames we have used the same PhysicsData for during resim
	float MaxPredictedInputs = 0;

	if (ComponentSettings.GetApplyInputDecayOverSetTime())
	{
		// If the decay curve should be applied over a set amount of time, calculate how many frames that time translates to
		MaxPredictedInputs = ComponentSettings.GetInputDecaySetTime() / BaseSolver->GetAsyncDeltaTime();
	}
	else
	{
		// Number of frames the input will predict and decay over until resimulation has completed
		MaxPredictedInputs = RewindData->GetLatestFrame() - 1 - LatestReceivedFrame;
	}

	// Linear decay
	const float PredictionAlpha = MaxPredictedInputs > 0 ? FMath::Clamp(NumPredictedInputs / MaxPredictedInputs, 0.0f, 1.0f) : 0.0f;

	// Get decay from curve
	float InputDecay = InputDecayCurve.GetRichCurveConst()->Eval(PredictionAlpha);

	// If enabled, scale input decay as a function of measured lag
	if (ComponentSettings.GetEnableLagScalingInputDecay())
	{
		const float ReferenceLagSeconds = ComponentSettings.GetInputDecayReferenceLagMs() * 0.001f;
		if (ReferenceLagSeconds >= UE_SMALL_NUMBER) // Setting ReferenceLagSeconds to 0 means don't scale with lag
		{
			const float ForwardPredictionTimeSeconds = GetForwardPredictionTime();
			if (ForwardPredictionTimeSeconds < UE_SMALL_NUMBER)
			{
				InputDecay = 0.0f; // no lag -> no decay
			}
			else if (ForwardPredictionTimeSeconds <= ReferenceLagSeconds)
			{
				InputDecay *= ForwardPredictionTimeSeconds / ReferenceLagSeconds;
			}
			else
			{
				// The formula below is designed to maintain the behavior tuned at ReferenceLagSeconds when we get higher lag
				// We interpret getting "similar behavior at higher lag" as maintaining the same amount of travel for a given input as that we get at the reference lag.
				// For instance, if the lag doubles relative to the reference lag, we want to travel half the distance per second for the same input, because we will be dead reckoning
				// for twice as long with a given input before being corrected. If d(x) is the function that gives us the decay, x being the lag
				// the modified input i(x) for that lag will be given by i(x) = I * (1 - d(x)), I being the unmodified received input.
				// We have d(L) = D, a value of decay D that feels good at the reference lag L
				// The equal distance at different lag can be written
				// (1 - d(x)) x = (1 - d(L)) L
				// So d(x) = 1 - L/x * (1-D)
				InputDecay = 1.0f - (ReferenceLagSeconds / ForwardPredictionTimeSeconds) * (1.0f - InputDecay);
			}
		}
	}

	return FMath::Clamp(InputDecay, 0.0f, 1.0f);
}

const float FAsyncNetworkPhysicsComponent::GetCurrentSimProxyInputDecayAtRuntime() const
{
	const FNetworkPhysicsSettingsNetworkPhysicsComponent& ComponentSettings = GetComponentSettings();
	float SimProxyRuntimeInputDecay = ComponentSettings.GetInputDecaySimProxyInputAtRuntime();
	if (ComponentSettings.GetEnableLagScalingSimProxyRuntimeInputDecay())
	{
		const float ReferenceLagSeconds = ComponentSettings.GetInputDecayReferenceLagMs() * 0.001f;
		if (ReferenceLagSeconds >= UE_SMALL_NUMBER) // Setting ReferenceLagSeconds to 0 means don't scale with lag
		{
			const float ForwardPredictionTimeSeconds = GetForwardPredictionTime();
			if (ForwardPredictionTimeSeconds < UE_SMALL_NUMBER)
			{
				SimProxyRuntimeInputDecay = 0.0f; // no lag -> no decay
			}
			else if (ForwardPredictionTimeSeconds <= ReferenceLagSeconds)
			{
				SimProxyRuntimeInputDecay *= ForwardPredictionTimeSeconds / ReferenceLagSeconds;
			}
			else
			{
				 // This formula is explained in GetCurrentInputDecay()
				SimProxyRuntimeInputDecay = 1.0f - (ReferenceLagSeconds / ForwardPredictionTimeSeconds) * (1.0f - SimProxyRuntimeInputDecay);
			}
		}
	}
	return FMath::Clamp(SimProxyRuntimeInputDecay, 0.0f, 1.0f);
}

const float FAsyncNetworkPhysicsComponent::GetForwardPredictionTime() const
{
	const Chaos::FPhysicsSolverBase* BaseSolver = GetSolver();
	if (const Chaos::FRewindData* RewindData = BaseSolver ? BaseSolver->GetRewindData() : nullptr)
	{
		const int32 LatestStateFrame = GetLatestReceivedStateFrame();
		if (LatestStateFrame != INDEX_NONE)
		{
			const int32 MeasuredInputDelayFrames = RewindData->GetLatestFrame() - LatestStateFrame;
			const float MeasuredInputDelaySeconds = FMath::Max(MeasuredInputDelayFrames, 0.0f) * BaseSolver->GetAsyncDeltaTime();
			return MeasuredInputDelaySeconds;
		}
	}
	return 0.0f;
}

void FAsyncNetworkPhysicsComponent::UpdateDynamicInputReplicationScaling()
{
	if (!PhysicsReplicationCVars::ResimulationCVars::bDynamicInputReplicationScalingEnabled)
	{
		InputsToNetwork_Owner = InputsToNetwork_OwnerDefault;
		return;
	}

	if (!IsServer())
	{
		return;
	}

	Chaos::FPBDRigidsSolver* RigidSolver = GetRigidSolver();
	if (!RigidSolver)
	{
		return;
	}

	const float TimeSinceLastDynamicScaling = RigidSolver->GetSolverTime() - TimeOfLastDynamicInputReplicationScaling;

	if (MissingInputCount > 0)
	{
		if (TimeSinceLastDynamicScaling > PhysicsReplicationCVars::ResimulationCVars::DynamicInputReplicationScalingIncreaseTimeInterval)
		{
			const int32 MaxScalable = FMath::CeilToInt32(PhysicsReplicationCVars::ResimulationCVars::DynamicInputReplicationScalingMaxInputsPercent / RigidSolver->GetAsyncDeltaTime());
			const uint16 MaxInputsValue = FMath::Min(NetworkPhysicsComponentConstants::MaxNumberOfElementsToNetwork, static_cast<uint16>(MaxScalable));

			// Increase the amount of inputs the owner sends
			InputsToNetwork_Owner++;

			// Update the average value for minimum clamping
			DynamicInputReplicationScalingAverageInputs += (static_cast<float>(InputsToNetwork_Owner) - DynamicInputReplicationScalingAverageInputs) * PhysicsReplicationCVars::ResimulationCVars::DynamicInputReplicationScalingIncreaseAverageMultiplier;
			
			// Clamp to maximum valid value
			InputsToNetwork_Owner = FMath::Min(InputsToNetwork_Owner, MaxInputsValue);

			TimeOfLastDynamicInputReplicationScaling = RigidSolver->GetSolverTime();
			MissingInputCount = 0;
		}
	}
	else if (TimeSinceLastDynamicScaling > PhysicsReplicationCVars::ResimulationCVars::DynamicInputReplicationScalingDecreaseTimeInterval)
	{
		// Decrease the amount of inputs the owner sends
		InputsToNetwork_Owner--;

		// Update the average value for minimum clamping, perform before clamping to allow for decreasing average even if the clamp might still round up.
		DynamicInputReplicationScalingAverageInputs += (static_cast<float>(InputsToNetwork_Owner) - DynamicInputReplicationScalingAverageInputs) * PhysicsReplicationCVars::ResimulationCVars::DynamicInputReplicationScalingDecreaseAverageMultiplier;

		// Clamp to minimum valid value
		const int32 MinScalable = FMath::Max(1, PhysicsReplicationCVars::ResimulationCVars::DynamicInputReplicationScalingMinInputs);
		const uint16 MinInputsValue = static_cast<uint16>(FMath::Max(FMath::RoundToInt(DynamicInputReplicationScalingAverageInputs), MinScalable));
		InputsToNetwork_Owner = FMath::Max(InputsToNetwork_Owner, MinInputsValue);

		TimeOfLastDynamicInputReplicationScaling = RigidSolver->GetSolverTime();
	}
}

float FAsyncNetworkPhysicsComponent::UpdateDynamicInputBufferScaling()
{
	if (!IsServer() || !InputHistory)
	{
		return InputBufferTargetSizeMs;
	}

	Chaos::FPBDRigidsSolver* RigidSolver = GetRigidSolver();
	if (!RigidSolver || FMath::IsNearlyZero(RigidSolver->GetAsyncDeltaTime()))
	{
		return InputBufferTargetSizeMs;
	}

	const float Dt = RigidSolver->GetAsyncDeltaTime();
	if (FMath::IsNearlyZero(Dt))
	{
		return InputBufferTargetSizeMs;
	}

	const int32 InputBufferSize = InputHistory->GetLatestFrame() - RigidSolver->GetCurrentFrame();
	if (InputBufferSize < 0)
	{
		// If buffer is negative, we are not yet setup
		return InputBufferTargetSizeMs;
	}

	// Cache fixed delta time as milliseconds (used for input buffer size calculations while Dt in seconds is used for time scaling / s
	const float FixedDeltaTimeMs = Dt * 1000.0f;

	const float MinInputBufferMs = FMath::Max(PhysicsReplicationCVars::ResimulationCVars::DynamicInputBufferScalingMinBufferMs, FixedDeltaTimeMs);
	const float MaxInputBufferMs = FMath::Max(PhysicsReplicationCVars::ResimulationCVars::DynamicInputBufferScalingMaxBufferMs, MinInputBufferMs + (FixedDeltaTimeMs * 2.0f));

	// Current input buffer size
	const float InputBufferTimeMs = static_cast<float>(InputBufferSize) * FixedDeltaTimeMs;

	// Running average input buffer size
	const float AverageOverSeconds = PhysicsReplicationCVars::ResimulationCVars::DynamicInputBufferScalingAverageTime > 0.0f ? PhysicsReplicationCVars::ResimulationCVars::DynamicInputBufferScalingAverageTime : 0.5f;
	const float AverageAlpha = FMath::Clamp(Dt / (Dt + AverageOverSeconds), 0.0f, 1.0f);
	InputBufferAverageMs += (InputBufferTimeMs - InputBufferAverageMs) * AverageAlpha;

	// Calculate what's currently considered the max good input buffer
	const float GoodInputBufferSize = MinInputBufferMs + FixedDeltaTimeMs + InputBufferDynamicMinSizeMs;

	// The input buffer is empty
	if (InputBufferSize <= 0)
	{
		// Bump up the target input buffer size
		const float BumpUpMultiplier = PhysicsReplicationCVars::ResimulationCVars::DynamicInputBufferScalingBumpUpMultiplier > 0.0f ? PhysicsReplicationCVars::ResimulationCVars::DynamicInputBufferScalingBumpUpMultiplier : 0.25f;
		InputBufferTargetSizeMs += FixedDeltaTimeMs * BumpUpMultiplier;

		// Bump up the dynamic minimum size
		InputBufferDynamicMinSizeMs += (InputBufferTargetSizeMs - InputBufferDynamicMinSizeMs) * BumpUpMultiplier;

#if !(UE_BUILD_SHIPPING)
		if (PhysicsReplicationCVars::ResimulationCVars::bDynamicInputBufferScalingDebugLogs)
		{
			UE_LOGF(LogChaos, Log, "Dynamic Input Buffer - EMPTY     InputBufferTargetSizeMs: %f - Added: %f - BumpUpMultiplier: %f"
				, InputBufferTargetSizeMs, (FixedDeltaTimeMs * BumpUpMultiplier), BumpUpMultiplier);
		}
#endif
	}
	// The input buffer is low
	else if(InputBufferTimeMs <= MinInputBufferMs)
	{
		// Scale up the target input buffer size
		const float ScaleUpMultiplier = PhysicsReplicationCVars::ResimulationCVars::DynamicInputBufferScalingScaleUpMultiplier > 0.0f ? PhysicsReplicationCVars::ResimulationCVars::DynamicInputBufferScalingScaleUpMultiplier : 1.0f;
		InputBufferTargetSizeMs += (ScaleUpMultiplier * FixedDeltaTimeMs * Dt);

		// Scale up the dynamic minimum size
		InputBufferDynamicMinSizeMs += (InputBufferTargetSizeMs - InputBufferDynamicMinSizeMs) * (ScaleUpMultiplier * Dt);

#if !(UE_BUILD_SHIPPING)
		if (PhysicsReplicationCVars::ResimulationCVars::bDynamicInputBufferScalingDebugLogs)
		{
			UE_LOGF(LogChaos, Log, "Dynamic Input Buffer - TOO LOW   InputBufferTimeMs: %f - InputBufferAverageMs %f - InputBufferTargetSizeMs: %f - ScaleUpMultiplier: %f"
				, InputBufferTimeMs, InputBufferAverageMs, InputBufferTargetSizeMs, ScaleUpMultiplier);
		}
#endif
	}
	// The input buffer is too large
	else if (InputBufferTimeMs > GoodInputBufferSize && InputBufferAverageMs > GoodInputBufferSize)
	{
		// Scale down the target input buffer size
		const float ScaleDownMinMultiplier = PhysicsReplicationCVars::ResimulationCVars::DynamicInputBufferScalingScaleDownMinMultiplier > 0.0f ? PhysicsReplicationCVars::ResimulationCVars::DynamicInputBufferScalingScaleDownMinMultiplier : 0.01f;
		const float ScaleDownMaxMultiplier = PhysicsReplicationCVars::ResimulationCVars::DynamicInputBufferScalingScaleDownMaxMultiplier > 0.0f ? PhysicsReplicationCVars::ResimulationCVars::DynamicInputBufferScalingScaleDownMaxMultiplier : 0.1f;

		const float OvershootMs = InputBufferAverageMs - MinInputBufferMs;
		const float OvershootAlpha = FMath::Clamp(OvershootMs / (MaxInputBufferMs - MinInputBufferMs), 0.0f, 1.0f);
		const float DecayRate = FMath::Lerp(ScaleDownMinMultiplier, ScaleDownMaxMultiplier, OvershootAlpha);
		InputBufferTargetSizeMs -= InputBufferTargetSizeMs * DecayRate * Dt;

#if !(UE_BUILD_SHIPPING)
		if (PhysicsReplicationCVars::ResimulationCVars::bDynamicInputBufferScalingDebugLogs)
		{
			UE_LOGF(LogChaos, Log, "Dynamic Input Buffer - TOO LARGE InputBufferTimeMs: %f - InputBufferAverageMs %f - GoodInputBufferSize: %f - InputBufferTargetSizeMs: %f - InputBufferDynamicMinSizeMs: %f - DecayRate: %f"
				, InputBufferTimeMs, InputBufferAverageMs, GoodInputBufferSize, InputBufferTargetSizeMs, InputBufferDynamicMinSizeMs, DecayRate);
		}
#endif
	}
	// The input buffer is good
	else
	{
		// Scale down the dynamic minimum size
		InputBufferDynamicMinSizeMs -= InputBufferDynamicMinSizeMs * Dt;

#if !(UE_BUILD_SHIPPING)
		if (PhysicsReplicationCVars::ResimulationCVars::bDynamicInputBufferScalingDebugLogs)
		{
			UE_LOGF(LogChaos, Log, "Dynamic Input Buffer - GOOD      InputBufferTimeMs: %f - InputBufferAverageMs %f - GoodInputBufferSize: %f - InputBufferTargetSizeMs: %f - InputBufferDynamicMinSizeMs: %f"
				, InputBufferTimeMs, InputBufferAverageMs, GoodInputBufferSize, InputBufferTargetSizeMs, InputBufferDynamicMinSizeMs);
		}
#endif
	}

	// Clamp input buffer target size between min and max
	InputBufferTargetSizeMs = FMath::Clamp(InputBufferTargetSizeMs, MinInputBufferMs, MaxInputBufferMs);

	return InputBufferTargetSizeMs;
}

void FAsyncNetworkPhysicsComponent::RegisterDataHistoryInRewindData()
{
	if (Chaos::FPhysicsSolverBase* BaseSolver = GetSolver())
	{
		if (Chaos::FRewindData* RewindData = BaseSolver->GetRewindData())
		{
			UnregisterDataHistoryFromRewindData();

			Chaos::FReadPhysicsObjectInterface_Internal Interface = Chaos::FPhysicsObjectInternalInterface::GetRead();
			Chaos::FGeometryParticleHandle* Particle = Interface.GetParticle(PhysicsObject);

			if (InputHistory)
			{
				RewindData->AddInputHistory(InputHistory, Particle);
			}
			if (StateHistory)
			{
				RewindData->AddStateHistory(StateHistory, Particle);
			}
		}
	}
}

void FAsyncNetworkPhysicsComponent::UnregisterDataHistoryFromRewindData()
{
	if (Chaos::FPhysicsSolverBase* BaseSolver = GetSolver())
	{
		if (Chaos::FRewindData* RewindData = BaseSolver->GetRewindData())
		{
			Chaos::FReadPhysicsObjectInterface_Internal Interface = Chaos::FPhysicsObjectInternalInterface::GetRead();
			Chaos::FGeometryParticleHandle* Particle = Interface.GetParticle(PhysicsObject);

			RewindData->RemoveInputHistory(InputHistory, Particle);
			RewindData->RemoveStateHistory(StateHistory, Particle);
		}
	}
}

const int32 FAsyncNetworkPhysicsComponent::SetupRewindData()
{
	int32 NumFrames = 0;

	if (Chaos::FPBDRigidsSolver* RigidSolver = GetRigidSolver())
	{
		NumFrames = FMath::Max<int32>(1, FMath::CeilToInt32((0.001f * Chaos::FPBDRigidsSolver::GetPhysicsHistoryTimeLength()) / RigidSolver->GetAsyncDeltaTime()));

		if (IsServer())
		{
			return NumFrames;
		}

		// Don't let this actor initialize RewindData if not using resimulation
		if (GetPhysicsReplicationMode() == EPhysicsReplicationMode::Resimulation)
		{
			if (RigidSolver->IsNetworkPhysicsPredictionEnabled() && RigidSolver->GetRewindData() == nullptr)
			{
				RigidSolver->EnableRewindCapture();
			}
		}

		if (Chaos::FRewindData* RewindData = RigidSolver->GetRewindData())
		{
			NumFrames = RewindData->Capacity();

			if (!DelegateOnRewindDataResize_Internal.IsValid())
			{
				DelegateOnRewindDataResize_Internal = RewindData->RewindDataResize.AddRaw(this, &FAsyncNetworkPhysicsComponent::OnRewindDataResize_Internal);
			}
		}
	}

	return NumFrames;
}

int32 FAsyncNetworkPhysicsComponent::GetLatestReceivedStateFrame() const
{
	if (LastReceivedStateFrame > INDEX_NONE)
	{
		return LastReceivedStateFrame;
	}
	else
	{
		const Chaos::FPhysicsSolverBase* BaseSolver = GetSolver();
		if (const Chaos::FRewindData* RewindData = BaseSolver ? BaseSolver->GetRewindData() : nullptr)
		{
			Chaos::FReadPhysicsObjectInterface_Internal Interface = Chaos::FPhysicsObjectInternalInterface::GetRead();
			if (Chaos::FGeometryParticleHandle* Particle = Interface.GetParticle(PhysicsObject))
			{
				return RewindData->GetLatestReceivedTargetStateFrame(*Particle);;
			}
		}
	}
	return INDEX_NONE;
}

void FAsyncNetworkPhysicsComponent::EnqueueImmediateActionInstance_Internal(TInstancedStruct<FNetworkPhysicsActionPayload>&& ActionInstance, const uint32 SourceId, const bool bReliable)
{
	EnqueueScheduledActionInstanceAtFrame_Internal(MoveTemp(ActionInstance), SourceId, -1, bReliable);
}

void FAsyncNetworkPhysicsComponent::EnqueueScheduledActionInstance_Internal(TInstancedStruct<FNetworkPhysicsActionPayload>&& ActionInstance, const uint32 SourceId, const float DelaySeconds, const bool bReliable)
{
	int32 LocalFrame = INDEX_NONE;
	if (Chaos::FPBDRigidsSolver* RigidSolver = GetRigidSolver())
	{
		if (RigidSolver->IsUsingFixedDt())
		{
			// Only apply a frame if we are actually delaying, else leave as INDEX_NONE which will queue up the action ASAP and handle edge-cases.
			int32 DelayFrames = FMath::CeilToInt32(DelaySeconds / RigidSolver->GetAsyncDeltaTime());
			if (DelayFrames > 0)
			{
				LocalFrame = RigidSolver->GetCurrentFrame() + DelayFrames;
			}
		}
	}
	EnqueueScheduledActionInstanceAtFrame_Internal(MoveTemp(ActionInstance), SourceId, LocalFrame, bReliable);
}

void FAsyncNetworkPhysicsComponent::EnqueueScheduledActionInstanceAtFrame_Internal(TInstancedStruct<FNetworkPhysicsActionPayload>&& ActionInstance, const uint32 SourceId, const int32 LocalFrame, const bool bReliable)
{
	FNetworkPhysicsActionPayload& Action = ActionInstance.GetMutable<FNetworkPhysicsActionPayload>();
	Action.SourceId = SourceId;

	// If we have a frame passed in, convert that to ServerFrame, if INDEX_NONE set ServerFrame to that which indicates that the action should be applied ASAP
	Action.ServerFrame = (LocalFrame == INDEX_NONE) ? INDEX_NONE : (LocalFrame + UE::NetworkPhysicsUtils::GetNetworkPhysicsTickOffset_Internal(GetSolver()));

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	if (PhysicsReplicationCVars::ResimulationCVars::bActionsEnableDebugLogs)
	{
		const bool bIsResim = GetEvolution() ? GetEvolution()->IsResimming() : false;

		UE_LOGF(LogChaos, Log, "PT %ls EnqueueActionAtFrame - Type: %ls - SourceId: %ld - LocalFrame: %d + Offset: %d = ServerFrame: %d - bIsResim: %d"
			, (IsServer() ? TEXT(" -- SERVER -- ") : TEXT(" ++ CLIENT ++ ")), (ActionInstance.GetScriptStruct() ? *ActionInstance.GetScriptStruct()->GetName() : TEXT("UNKNOWN ACTION"))
			, Action.SourceId, LocalFrame, UE::NetworkPhysicsUtils::GetNetworkPhysicsTickOffset_Internal(GetSolver()), Action.ServerFrame, bIsResim);
	}
#endif

	RegisterAction_Internal(ActionInstance, /*bWasReceivedFromReplication*/ false, bReliable);
}

namespace UE::NetworkPhysicsUtils
{
	int32 GetUpcomingServerFrame_External(UWorld* World)
	{
		if (World)
		{
			if (FPhysScene* Scene = World->GetPhysicsScene())
			{
				if (Chaos::FPhysicsSolver* Solver = Scene->GetSolver())
				{
					int32 ServerFrame = Solver->GetMarshallingManager().GetInternalStep_External();
					ServerFrame += GetNetworkPhysicsTickOffset_External(World);
					return ServerFrame;
				}
			}
		}

		return INDEX_NONE;
	}

	const int32 GetNetworkPhysicsTickOffset_External(const UWorld* World)
	{
		Chaos::EnsureIsInGameThreadContext();
		if (World)
		{
			if (const FPhysScene* Scene = World->GetPhysicsScene())
			{
				if (const IPhysicsReplication* ReplicationSystem = Scene->GetPhysicsReplication())
				{
					return ReplicationSystem->GetNetworkPhysicsTickOffset();
				}
			}
		}

		return 0;
	}

	const int32 GetNetworkPhysicsTickOffset_Internal(const Chaos::FPhysicsSolverBase* Solver)
	{
		Chaos::EnsureIsInPhysicsThreadContext();
		if (Solver)
		{
			if (const IPhysicsReplicationAsync* ReplicationSystem = Solver->GetPhysicsReplication_Internal())
			{
				return ReplicationSystem->GetNetworkPhysicsTickOffset_Internal();
			}
		}

		return 0;
	}

	const UObject* GetOwnerFromUObject_External(const UObject* SourceObject)
	{
		if (!IsValid(SourceObject))
		{
			return nullptr;
		}

		if (const AActor* Actor = Cast<AActor>(SourceObject))
		{
			return Actor;
		}

		if (const UActorComponent* Component = Cast<UActorComponent>(SourceObject))
		{
			if (const AActor* Owner = Component->GetOwner())
			{
				return Owner;
			}
		}

		if (const AActor* OuterActor = SourceObject->GetTypedOuter<AActor>())
		{
			return OuterActor;
		}

		return SourceObject;
	}

	const uint32 GetNetworkStableHash_External(const UObject* InSourceObject)
	{
		const UObject* OwnerObject = GetOwnerFromUObject_External(InSourceObject);
		if (!IsValid(OwnerObject))
		{
			return 0u;
		}

		// If object is replicated, use Iris NetRepHandle
		if (const UWorld* World = OwnerObject->GetWorld())
		{
			if (UEngineReplicationBridge* ReplicationBridge = UE::Net::FReplicationSystemUtil::GetEngineReplicationBridge(World))
			{
				// If your branch scopes this enum differently, adjust this token only.
				const UE::Net::FNetRefHandle NetRefHandle = ReplicationBridge->GetReplicatedRefHandle(OwnerObject, UE::Net::EGetRefHandleFlags::None);

				if (NetRefHandle.IsValid())
				{
					return UE::Net::GetTypeHash(NetRefHandle);
				}
			}
		}

		// If object has a network stable name, that that
		if (OwnerObject->IsNameStableForNetworking())
		{
			const UObject* PackageOuter = OwnerObject->GetOutermost();
			const FString StablePath = OwnerObject->GetPathName(PackageOuter);

			if (!StablePath.IsEmpty())
			{
				return FCrc::StrCrc32(*StablePath);
			}
		}

		// No network stable option to use as a SourceId
		return 0u;
	}
}