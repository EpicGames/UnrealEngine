// Copyright Epic Games, Inc. All Rights Reserved.

#include "Physics/NetworkPhysicsComponentNetSerializers.h"
#include UE_INLINE_GENERATED_CPP_BY_NAME(NetworkPhysicsComponentNetSerializers)

#include "Iris/ReplicationState/PropertyNetSerializerInfoRegistry.h"
#include "Iris/ReplicationState/ReplicationStateDescriptorBuilder.h"
#include "Iris/Serialization/NetBitStreamReader.h"
#include "Iris/Serialization/NetBitStreamUtil.h"
#include "Iris/Serialization/NetBitStreamWriter.h"
#include "Iris/Serialization/NetSerializerArrayStorage.h"
#include "Iris/Serialization/NetSerializerDelegates.h"
#include "Iris/Serialization/NetSerializers.h"
#include "Iris/Serialization/PackedIntNetSerializers.h"
#include "Misc/ScopeExit.h"
#include "Misc/ScopeRWLock.h"
#include "Net/Core/Trace/NetTrace.h"
#include "Physics/NetworkPhysicsComponent.h"
#include "Templates/IsPODType.h"
#include "UObject/UObjectIterator.h"

// Forward declarations so TIsPODType can be specialized before FNetSerializerArrayStorage<T> is instantiated.
namespace UE::Net
{
	struct FNetworkPhysicsActionElementQuantized;
	struct FNetworkPhysicsElementQuantized;
	struct FNetworkPhysicsPayloadQuantized;
	struct FNetworkPhysicsActionPayloadQuantized;
	struct FNetworkPhysicsDataCollectionQuantized;
	struct FNetworkPhysicsActionCollectionQuantized;
}

// ---- POD (Plain Old Data) declarations (required by FNetSerializerArrayStorage) ----------------
template<> struct TIsPODType<UE::Net::FNetworkPhysicsActionElementQuantized> { enum { Value = true }; };
template<> struct TIsPODType<UE::Net::FNetworkPhysicsElementQuantized> { enum { Value = true }; };
template<> struct TIsPODType<UE::Net::FNetworkPhysicsPayloadQuantized> { enum { Value = true }; };
template<> struct TIsPODType<UE::Net::FNetworkPhysicsActionPayloadQuantized> { enum { Value = true }; };
template<> struct TIsPODType<UE::Net::FNetworkPhysicsDataCollectionQuantized> { enum { Value = true }; };
template<> struct TIsPODType<UE::Net::FNetworkPhysicsActionCollectionQuantized> { enum { Value = true }; };


namespace UE::Net
{

// ============================================================
// Internal quantized types
// ============================================================

// Per-element quantized data for a collection array element.
// StructData holds the quantized form of the concrete derived struct.
// StructName is the fully-qualified path name used to identify and look up the concrete type.
// StructDescriptorTraits is used to skip dynamic-state calls when the struct has none.
struct FNetworkPhysicsElementQuantized
{
	FNetSerializerAlignedStorage StructData;
	FName StructName;
	EReplicationStateTraits StructDescriptorTraits;
};

// Per-element quantized data for an action collection element. Identical layout.
struct FNetworkPhysicsActionElementQuantized
{
	FNetSerializerAlignedStorage StructData;
	FName StructName;
	EReplicationStateTraits StructDescriptorTraits;
};

// Payload base-struct quantized form: just the frame counter stored as ServerFrame+1 (unsigned).
// Zero-initialized state (QuantizedServerFrame==0) represents ServerFrame==-1 (INDEX_NONE).
struct FNetworkPhysicsPayloadQuantized
{
	uint32 QuantizedServerFrame;
};

// Action payload base-struct quantized form: same frame encoding plus ActionId.
struct FNetworkPhysicsActionPayloadQuantized
{
	uint32 QuantizedServerFrame;
	uint32 SourceId;
	uint16 ActionId;
};

// Quantized form for a full data collection.
struct FNetworkPhysicsDataCollectionQuantized
{
	FNetSerializerArrayStorage<FNetworkPhysicsElementQuantized> Elements;
};

// Quantized form for a full action collection.
struct FNetworkPhysicsActionCollectionQuantized
{
	FNetSerializerArrayStorage<FNetworkPhysicsActionElementQuantized> Elements;
};

} // namespace UE::Net

namespace UE::Net
{

// Forward-declared helpers - definitions are below with the collection serializers.
static void WritePackedUint32(FNetSerializationContext& Context, const FNetSerializeArgs& BaseArgs, uint32 Value);
static void WritePackedUint32(FNetSerializationContext& Context, const FNetSerializeDeltaArgs& BaseArgs, uint32 Value);
static uint32 ReadPackedUint32(FNetSerializationContext& Context, const FNetDeserializeArgs& BaseArgs);
static uint32 ReadPackedUint32(FNetSerializationContext& Context, const FNetDeserializeDeltaArgs& BaseArgs);


// ============================================================
// FNetworkPhysicsDescriptorCache
// ============================================================
// Caches replication state descriptors, struct type hashes, UScriptStruct* pointers, and custom serializer lookups.
// Replaces FInstancedStructDescriptorCache (IrisCore-internal, not exported) using the
// IRISCORE_API FReplicationStateDescriptorBuilder::CreateDescriptorForStruct.
// Thread-safe via FRWLock - reads are concurrent, writes are exclusive and rare (only on first encounter of a new type).
// Each collection serializer (DataCollection, ActionCollection) has its own static instance.
struct FNetworkPhysicsDescriptorCache
{
	TRefCountPtr<const FReplicationStateDescriptor> FindDescriptor(FName StructPath) const
	{
		FReadScopeLock ReadLock(Lock);
		const TRefCountPtr<const FReplicationStateDescriptor>* Found = Descriptors.Find(StructPath);
		return Found ? *Found : TRefCountPtr<const FReplicationStateDescriptor>{};
	}

	TRefCountPtr<const FReplicationStateDescriptor> FindOrAddDescriptor(const UScriptStruct* Struct)
	{
		const FName PathFName(Struct->GetPathName());
		{
			FReadScopeLock ReadLock(Lock);
			if (const TRefCountPtr<const FReplicationStateDescriptor>* Found = Descriptors.Find(PathFName))
			{
				return *Found;
			}
		}
		// Build outside the lock - CreateDescriptorForStruct may perform UObject reflection work
		TRefCountPtr<const FReplicationStateDescriptor> NewDescriptor =
			FReplicationStateDescriptorBuilder::CreateDescriptorForStruct(Struct);
		if (NewDescriptor.IsValid())
		{
			FWriteScopeLock WriteLock(Lock);
			TRefCountPtr<const FReplicationStateDescriptor>& Slot = Descriptors.FindOrAdd(PathFName);
			if (!Slot.IsValid())
			{
				Slot = NewDescriptor;
			}
			RegisterPath(PathFName, Struct);
			return Slot;
		}
		return NewDescriptor;
	}

	TRefCountPtr<const FReplicationStateDescriptor> FindOrAddDescriptor(FName StructPath)
	{
		{
			FReadScopeLock ReadLock(Lock);
			if (const TRefCountPtr<const FReplicationStateDescriptor>* Found = Descriptors.Find(StructPath))
			{
				return *Found;
			}
		}
		const UScriptStruct* Struct = FindOrCacheStruct(StructPath);
		if (Struct)
		{
			return FindOrAddDescriptor(Struct);
		}
		return {};
	}

	// Deterministic 32-bit hash for a struct path name (goes on the wire, must be consistent across processes).
	uint32 GetStructNameHash(FName StructPath)
	{
		{
			FReadScopeLock ReadLock(Lock);
			if (const uint32* Found = PathToHash.Find(StructPath))
			{
				return *Found;
			}
		}
		const uint32 Hash = FCrc::StrCrc32(*StructPath.ToString());
		FWriteScopeLock WriteLock(Lock);
		PathToHash.Add(StructPath, Hash);
		RegisterHash(Hash, StructPath);
		return Hash;
	}

	// Resolve a 32-bit wire hash back to the FName path.
	// On first miss, scans loaded subtypes of FNetworkPhysicsPayload/FNetworkPhysicsActionPayload to build the hash map
	FName FindNameByHash(uint32 Hash)
	{
		{
			FReadScopeLock ReadLock(Lock);
			if (const FName* Found = HashToPath.Find(Hash))
			{
				return *Found;
			}
		}

		if (!bDidFullScan)
		{
			// Build the scan results into a local map OUTSIDE the lock to avoid blocking readers
			TMap<uint32, FName> ScannedHashes;
			TMap<FName, const UScriptStruct*> ScannedStructs;
			{
				const UScriptStruct* PayloadBase = FNetworkPhysicsPayload::StaticStruct();
				const UScriptStruct* ActionPayloadBase = FNetworkPhysicsActionPayload::StaticStruct();
				for (TObjectIterator<UScriptStruct> It; It; ++It)
				{
					if (It->IsChildOf(PayloadBase) || It->IsChildOf(ActionPayloadBase))
					{
						const FName PathFName(It->GetPathName());
						const uint32 StructHash = FCrc::StrCrc32(*PathFName.ToString());
						ScannedHashes.Add(StructHash, PathFName);
						ScannedStructs.Add(PathFName, *It);
					}
				}
			}

			// Merge scan results under a brief write lock
			FWriteScopeLock WriteLock(Lock);
			if (!bDidFullScan)
			{
				for (const TPair<uint32, FName>& Pair : ScannedHashes)
				{
					RegisterHash(Pair.Key, Pair.Value);
				}
				for (const TPair<FName, const UScriptStruct*>& Pair : ScannedStructs)
				{
					PathToHash.Add(Pair.Key, FCrc::StrCrc32(*Pair.Key.ToString()));
					StructCache.Add(Pair.Key, Pair.Value);
				}
				bDidFullScan = true;
			}
			if (const FName* Found = HashToPath.Find(Hash))
			{
				return *Found;
			}
		}
		return FName();
	}

	// Cached UScriptStruct* lookup - avoids repeated FindObject on hot paths.
	const UScriptStruct* FindOrCacheStruct(FName StructPath)
	{
		{
			FReadScopeLock ReadLock(Lock);
			if (const UScriptStruct* const* Found = StructCache.Find(StructPath))
			{
				return *Found;
			}
		}
		// FindObject happens outside the lock (UObject system is not lock-safe)
		const UScriptStruct* Struct = FindObject<UScriptStruct>(nullptr, *StructPath.ToString());
		if (Struct)
		{
			FWriteScopeLock WriteLock(Lock);
			StructCache.Add(StructPath, Struct);
		}
		return Struct;
	}

	// Cached custom serializer lookup by path name - avoids per-element string ops.
	// Returns nullptr if no custom serializer is registered (caller should use FStructNetSerializer).
	const FNetSerializer* FindCustomSerializer(FName StructPath)
	{
		{
			FReadScopeLock ReadLock(Lock);
			if (const FNetSerializer* const* Found = CustomSerializerCache.Find(StructPath))
			{
				return *Found;
			}
		}
		// Resolve outside the lock, then cache the result (including nullptr = "no custom serializer")
		const FNetSerializer* Result = ResolveCustomSerializer(StructPath);
		FWriteScopeLock WriteLock(Lock);
		CustomSerializerCache.Add(StructPath, Result);
		return Result;
	}

	// Cached UScriptStruct* -> path FName lookup - avoids per-element GetPathName FString alloc + FName construction.
	// First call per type pays the same cost as today; all subsequent calls are a TMap hit.
	FName GetCachedPathName(const UScriptStruct* Struct)
	{
		if (!Struct)
		{
			return FName();
		}
		{
			FReadScopeLock ReadLock(Lock);
			if (const FName* Found = PathNameByStruct.Find(Struct))
			{
				return *Found;
			}
		}
		const FName PathFName(Struct->GetPathName());
		FWriteScopeLock WriteLock(Lock);
		// RegisterPath populates PathNameByStruct alongside HashToPath / PathToHash / StructCache.
		RegisterPath(PathFName, Struct);
		return PathFName;
	}

private:
	// Must be called under write lock
	void RegisterPath(FName PathFName, const UScriptStruct* Struct)
	{
		const uint32 Hash = FCrc::StrCrc32(*PathFName.ToString());
		RegisterHash(Hash, PathFName);
		PathToHash.Add(PathFName, Hash);
		StructCache.Add(PathFName, Struct);
		PathNameByStruct.Add(Struct, PathFName);
	}

	// Must be called under write lock
	void RegisterHash(uint32 Hash, FName PathFName)
	{
		if (const FName* Existing = HashToPath.Find(Hash))
		{
			ensureMsgf(*Existing == PathFName, TEXT("FNetworkPhysicsDescriptorCache: CRC32 hash collision between '%s' and '%s'"),
				ToCStr(Existing->ToString()), ToCStr(PathFName.ToString()));
			return; // Do not overwrite existing mapping on collision
		}
		HashToPath.Add(Hash, PathFName);
	}

	// Look up a custom serializer by full path name (e.g. "/Script/Engine.FNetworkPhysicsPayload").
	// The registry expects the simple struct name (after the last dot), not the full path.
	static const FNetSerializer* ResolveCustomSerializer(FName PathName)
	{
		if (PathName.IsNone())
		{
			return nullptr;
		}
		const FString PathStr = PathName.ToString();
		int32 DotIdx = INDEX_NONE;
		PathStr.FindLastChar(TEXT('.'), DotIdx);
		const FName SimpleName = (DotIdx != INDEX_NONE) ? FName(PathStr.Mid(DotIdx + 1)) : PathName;
		const FPropertyNetSerializerInfo* Info = FPropertyNetSerializerInfoRegistry::FindStructSerializerInfo(SimpleName);
		return Info ? Info->GetNetSerializer(nullptr) : nullptr;
	}

	mutable FRWLock Lock;
	TMap<FName, TRefCountPtr<const FReplicationStateDescriptor>> Descriptors;
	TMap<uint32, FName> HashToPath;
	TMap<FName, uint32> PathToHash;
	TMap<FName, const UScriptStruct*> StructCache;
	TMap<FName, const FNetSerializer*> CustomSerializerCache;
	TMap<const UScriptStruct*, FName> PathNameByStruct;
	bool bDidFullScan = false;
};


// ============================================================
// Shared collection helpers
// ============================================================

// Free per-element dynamic state then free the element's StructData storage.
template<typename ElementType>
static void FreeElement(FNetSerializationContext& Context, ElementType& Element, FNetworkPhysicsDescriptorCache& Cache)
{
	if (Element.StructData.Num() > 0 && EnumHasAnyFlags(Element.StructDescriptorTraits, EReplicationStateTraits::HasDynamicState))
	{
		const FNetSerializer* CustomSerializer = Cache.FindCustomSerializer(Element.StructName);
		if (CustomSerializer && EnumHasAnyFlags(CustomSerializer->Traits, ENetSerializerTraits::HasDynamicState))
		{
			FNetFreeDynamicStateArgs FreeArgs;
			FreeArgs.NetSerializerConfig = CustomSerializer->DefaultConfig;
			FreeArgs.Source = NetSerializerValuePointer(Element.StructData.GetData());
			CustomSerializer->FreeDynamicState(Context, FreeArgs);
		}
		else if (!CustomSerializer)
		{
			FStructNetSerializerConfig StructConfig;
			StructConfig.StateDescriptor = Cache.FindDescriptor(Element.StructName);
			if (StructConfig.StateDescriptor.IsValid())
			{
				FNetFreeDynamicStateArgs FreeArgs;
				FreeArgs.NetSerializerConfig = &StructConfig;
				FreeArgs.Source = NetSerializerValuePointer(Element.StructData.GetData());
				UE_NET_GET_SERIALIZER(FStructNetSerializer).FreeDynamicState(Context, FreeArgs);
			}
		}
	}
	Element.StructData.Free(Context);
}

// Write a struct type as a compact 32-bit CRC hash (deterministic across processes).
static void WriteStructNameHash(FNetBitStreamWriter& Writer, FName StructName, FNetworkPhysicsDescriptorCache& Cache)
{
	Writer.WriteBits(Cache.GetStructNameHash(StructName), 32u);
}

// Read a 32-bit struct type hash and resolve to FName.
static FName ReadStructNameHash(FNetBitStreamReader& Reader, FNetworkPhysicsDescriptorCache& Cache)
{
	const uint32 Hash = Reader.ReadBits(32u);
	return Cache.FindNameByHash(Hash);
}

// Cached reference to the packed uint32 serializer - used by all count/frame write/read helpers.
static const FNetSerializer& GetPackedUint32Serializer()
{
	return UE_NET_GET_SERIALIZER(FPackedUint32NetSerializer);
}

// Write a uint32 value using packed (variable-length) encoding.
static void WritePackedUint32(FNetSerializationContext& Context, const FNetSerializeArgs& BaseArgs, uint32 Value)
{
	const FNetSerializer& PackedSerializer = GetPackedUint32Serializer();
	FNetSerializeArgs PackedArgs = BaseArgs;
	PackedArgs.Source = NetSerializerValuePointer(&Value);
	PackedArgs.NetSerializerConfig = PackedSerializer.DefaultConfig;
	PackedSerializer.Serialize(Context, PackedArgs);
}

// Overload accepting delta args (used by SerializeDelta functions).
static void WritePackedUint32(FNetSerializationContext& Context, const FNetSerializeDeltaArgs& BaseArgs, uint32 Value)
{
	const FNetSerializer& PackedSerializer = GetPackedUint32Serializer();
	FNetSerializeArgs PackedArgs;
	PackedArgs.Version = BaseArgs.Version;
	PackedArgs.Source = NetSerializerValuePointer(&Value);
	PackedArgs.NetSerializerConfig = PackedSerializer.DefaultConfig;
	PackedSerializer.Serialize(Context, PackedArgs);
}

// Read a uint32 value using packed (variable-length) encoding.
static uint32 ReadPackedUint32(FNetSerializationContext& Context, const FNetDeserializeArgs& BaseArgs)
{
	const FNetSerializer& PackedSerializer = GetPackedUint32Serializer();
	uint32 Value = 0u;
	FNetDeserializeArgs PackedArgs = BaseArgs;
	PackedArgs.Target = NetSerializerValuePointer(&Value);
	PackedArgs.NetSerializerConfig = PackedSerializer.DefaultConfig;
	PackedSerializer.Deserialize(Context, PackedArgs);
	return Value;
}

// Overload accepting delta args (used by DeserializeDelta functions).
static uint32 ReadPackedUint32(FNetSerializationContext& Context, const FNetDeserializeDeltaArgs& BaseArgs)
{
	const FNetSerializer& PackedSerializer = GetPackedUint32Serializer();
	uint32 Value = 0u;
	FNetDeserializeArgs PackedArgs;
	PackedArgs.Version = BaseArgs.Version;
	PackedArgs.Target = NetSerializerValuePointer(&Value);
	PackedArgs.NetSerializerConfig = PackedSerializer.DefaultConfig;
	PackedSerializer.Deserialize(Context, PackedArgs);
	return Value;
}

// Templatized helpers shared by both DataCollection and ActionCollection serializers.
// Both element quantized types have identical layout (StructData, StructName, StructDescriptorTraits).
template<typename QuantizedCollectionType, typename ElementType>
static bool IsEqualImpl(const FNetIsEqualArgs& Args)
{
	if (Args.bStateIsQuantized)
	{
		const QuantizedCollectionType& V0 = *reinterpret_cast<const QuantizedCollectionType*>(Args.Source0);
		const QuantizedCollectionType& V1 = *reinterpret_cast<const QuantizedCollectionType*>(Args.Source1);
		if (V0.Elements.Num() != V1.Elements.Num())
		{
			return false;
		}

		const ElementType* E0 = V0.Elements.GetData();
		const ElementType* E1 = V1.Elements.GetData();
		for (uint32 Idx = 0u; Idx < V0.Elements.Num(); ++Idx)
		{
			if (E0[Idx].StructName != E1[Idx].StructName)
			{
				return false;
			}
			if (E0[Idx].StructData.Num() != E1[Idx].StructData.Num())
			{
				return false;
			}
			if (FMemory::Memcmp(E0[Idx].StructData.GetData(), E1[Idx].StructData.GetData(), E0[Idx].StructData.Num()) != 0)
			{
				return false;
			}
		}
		return true;
	}
	return false;
}

template<typename QuantizedCollectionType, typename ElementType>
static void CloneDynamicStateImpl(FNetSerializationContext& Context, const FNetCloneDynamicStateArgs& Args, FNetworkPhysicsDescriptorCache& Cache)
{
	const QuantizedCollectionType& Source = *reinterpret_cast<const QuantizedCollectionType*>(Args.Source);
	QuantizedCollectionType& Target = *reinterpret_cast<QuantizedCollectionType*>(Args.Target);

	Target.Elements.Clone(Context, Source.Elements);

	const ElementType* SrcElements = Source.Elements.GetData();
	ElementType* DstElements = Target.Elements.GetData();
	const uint32 Count = Source.Elements.Num();

	for (uint32 Idx = 0u; Idx < Count; ++Idx)
	{
		DstElements[Idx].StructData.Clone(Context, SrcElements[Idx].StructData);

		if (EnumHasAnyFlags(SrcElements[Idx].StructDescriptorTraits, EReplicationStateTraits::HasDynamicState))
		{
			// Dispatch to custom serializer if registered, otherwise FStructNetSerializer (mirrors FreeElement).
			const FNetSerializer* CustomSerializer = Cache.FindCustomSerializer(SrcElements[Idx].StructName);
			if (CustomSerializer && EnumHasAnyFlags(CustomSerializer->Traits, ENetSerializerTraits::HasDynamicState))
			{
				FNetCloneDynamicStateArgs CloneArgs = Args;
				CloneArgs.NetSerializerConfig = CustomSerializer->DefaultConfig;
				CloneArgs.Source = NetSerializerValuePointer(SrcElements[Idx].StructData.GetData());
				CloneArgs.Target = NetSerializerValuePointer(DstElements[Idx].StructData.GetData());
				CustomSerializer->CloneDynamicState(Context, CloneArgs);
			}
			else if (!CustomSerializer)
			{
				FStructNetSerializerConfig StructConfig;
				StructConfig.StateDescriptor = Cache.FindDescriptor(SrcElements[Idx].StructName);
				if (StructConfig.StateDescriptor.IsValid())
				{
					FNetCloneDynamicStateArgs CloneArgs = Args;
					CloneArgs.NetSerializerConfig = &StructConfig;
					CloneArgs.Source = NetSerializerValuePointer(SrcElements[Idx].StructData.GetData());
					CloneArgs.Target = NetSerializerValuePointer(DstElements[Idx].StructData.GetData());
					UE_NET_GET_SERIALIZER(FStructNetSerializer).CloneDynamicState(Context, CloneArgs);
				}
			}
		}
	}
}

template<typename QuantizedCollectionType, typename ElementType>
static void FreeDynamicStateImpl(FNetSerializationContext& Context, const FNetFreeDynamicStateArgs& Args, FNetworkPhysicsDescriptorCache& Cache)
{
	QuantizedCollectionType& Value = *reinterpret_cast<QuantizedCollectionType*>(Args.Source);
	ElementType* Elements = Value.Elements.GetData();
	const uint32 Count = Value.Elements.Num();
	for (uint32 Idx = 0u; Idx < Count; ++Idx)
	{
		FreeElement(Context, Elements[Idx], Cache);
	}
	Value.Elements.Free(Context);
}

template<typename QuantizedCollectionType, typename ElementType>
static void CollectNetReferencesImpl(FNetSerializationContext& Context, const FNetCollectReferencesArgs& Args, FNetworkPhysicsDescriptorCache& Cache)
{
	const QuantizedCollectionType& Value = *reinterpret_cast<const QuantizedCollectionType*>(Args.Source);
	const uint32 Count = Value.Elements.Num();
	const ElementType* Elements = Value.Elements.GetData();

	for (uint32 Idx = 0u; Idx < Count; ++Idx)
	{
		const ElementType& El = Elements[Idx];
		if (El.StructName.IsNone())
		{
			continue;
		}

		// Dispatch to custom serializer if registered, otherwise FStructNetSerializer (mirrors FreeElement).
		// Custom serializers may internally hold object references not reflected in StructDescriptorTraits.
		const FNetSerializer* CustomSerializer = Cache.FindCustomSerializer(El.StructName);
		if (CustomSerializer && EnumHasAnyFlags(CustomSerializer->Traits, ENetSerializerTraits::HasCustomNetReference))
		{
			FNetCollectReferencesArgs ElementArgs = Args;
			ElementArgs.NetSerializerConfig = CustomSerializer->DefaultConfig;
			ElementArgs.Source = NetSerializerValuePointer(El.StructData.GetData());
			CustomSerializer->CollectNetReferences(Context, ElementArgs);
		}
		else if (!CustomSerializer)
		{
			if (!EnumHasAnyFlags(El.StructDescriptorTraits, EReplicationStateTraits::HasObjectReference))
			{
				continue;
			}

			FStructNetSerializerConfig StructConfig;
			StructConfig.StateDescriptor = Cache.FindDescriptor(El.StructName);
			if (!StructConfig.StateDescriptor.IsValid())
			{
				continue;
			}

			FNetCollectReferencesArgs ElementArgs = Args;
			ElementArgs.NetSerializerConfig = &StructConfig;
			ElementArgs.Source = NetSerializerValuePointer(El.StructData.GetData());
			UE_NET_GET_SERIALIZER(FStructNetSerializer).CollectNetReferences(Context, ElementArgs);
		}
	}
}


// ============================================================
// FNetworkPhysicsPayloadNetSerializer
// ============================================================

struct FNetworkPhysicsPayloadNetSerializer
{
	static constexpr uint32 Version = 0;

	typedef FNetworkPhysicsPayload SourceType;
	typedef FNetworkPhysicsPayloadQuantized QuantizedType;
	typedef FNetworkPhysicsPayloadNetSerializerConfig ConfigType;

	inline static const ConfigType DefaultConfig;

	static void Serialize(FNetSerializationContext&, const FNetSerializeArgs&);
	static void Deserialize(FNetSerializationContext&, const FNetDeserializeArgs&);

	static void SerializeDelta(FNetSerializationContext&, const FNetSerializeDeltaArgs&);
	static void DeserializeDelta(FNetSerializationContext&, const FNetDeserializeDeltaArgs&);

	static void Quantize(FNetSerializationContext&, const FNetQuantizeArgs&);
	static void Dequantize(FNetSerializationContext&, const FNetDequantizeArgs&);

	static bool IsEqual(FNetSerializationContext&, const FNetIsEqualArgs&);
	static bool Validate(FNetSerializationContext&, const FNetValidateArgs&);

};

UE_NET_IMPLEMENT_SERIALIZER(FNetworkPhysicsPayloadNetSerializer);

// Full serialize: write QuantizedServerFrame as packed uint32.
void FNetworkPhysicsPayloadNetSerializer::Serialize(FNetSerializationContext& Context, const FNetSerializeArgs& Args)
{
	const QuantizedType& Value = *reinterpret_cast<const QuantizedType*>(Args.Source);

	FNetBitStreamWriter* Writer = Context.GetBitStreamWriter();
	UE_NET_TRACE_SCOPE(ServerFrame, *Writer, Context.GetTraceCollector(), ENetTraceVerbosity::Verbose);

	WritePackedUint32(Context, Args, Value.QuantizedServerFrame);
}

// Full deserialize: read QuantizedServerFrame as packed uint32.
void FNetworkPhysicsPayloadNetSerializer::Deserialize(FNetSerializationContext& Context, const FNetDeserializeArgs& Args)
{
	QuantizedType& Value = *reinterpret_cast<QuantizedType*>(Args.Target);
	Value.QuantizedServerFrame = ReadPackedUint32(Context, Args);
}

// Delta serialize: 1 bit bIncrementalFrame, else sign + packed |delta|.
void FNetworkPhysicsPayloadNetSerializer::SerializeDelta(FNetSerializationContext& Context, const FNetSerializeDeltaArgs& Args)
{
	const QuantizedType& Value = *reinterpret_cast<const QuantizedType*>(Args.Source);
	const QuantizedType& PrevValue = *reinterpret_cast<const QuantizedType*>(Args.Prev);

	FNetBitStreamWriter* Writer = Context.GetBitStreamWriter();
	UE_NET_TRACE_SCOPE(ServerFrame, *Writer, Context.GetTraceCollector(), ENetTraceVerbosity::Verbose);

	// QuantizedServerFrame stores (ServerFrame + 1) so delta==1 means consecutive frame
	const bool bIncrementalFrame = (Value.QuantizedServerFrame == PrevValue.QuantizedServerFrame + 1u);
	Writer->WriteBits(bIncrementalFrame, 1u);

	if (!bIncrementalFrame)
	{
		const int64 Delta = static_cast<int64>(Value.QuantizedServerFrame) - static_cast<int64>(PrevValue.QuantizedServerFrame);
		const bool bNegative = Delta < 0;
		Writer->WriteBits(bNegative, 1u);
		WritePackedUint32(Context, Args, static_cast<uint32>(FMath::Abs(Delta)));
	}
}

// Delta deserialize: inverse of SerializeDelta.
void FNetworkPhysicsPayloadNetSerializer::DeserializeDelta(FNetSerializationContext& Context, const FNetDeserializeDeltaArgs& Args)
{
	QuantizedType& Value = *reinterpret_cast<QuantizedType*>(Args.Target);
	const QuantizedType& PrevValue = *reinterpret_cast<const QuantizedType*>(Args.Prev);

	FNetBitStreamReader* Reader = Context.GetBitStreamReader();

	const bool bIncrementalFrame = (Reader->ReadBits(1u) != 0u);
	if (bIncrementalFrame)
	{
		Value.QuantizedServerFrame = PrevValue.QuantizedServerFrame + 1u;
	}
	else
	{
		const bool bNegative = (Reader->ReadBits(1u) != 0u);
		const uint32 AbsDelta = ReadPackedUint32(Context, Args);
		const int64 SignedDelta = bNegative ? -static_cast<int64>(AbsDelta) : static_cast<int64>(AbsDelta);
		Value.QuantizedServerFrame = static_cast<uint32>(static_cast<int64>(PrevValue.QuantizedServerFrame) + SignedDelta);
	}
}

// Quantize: store ServerFrame+1 as unsigned (0 == INDEX_NONE, matches legacy encoding).
void FNetworkPhysicsPayloadNetSerializer::Quantize(FNetSerializationContext&, const FNetQuantizeArgs& Args)
{
	const SourceType& Source = *reinterpret_cast<const SourceType*>(Args.Source);
	QuantizedType& Target = *reinterpret_cast<QuantizedType*>(Args.Target);

	Target.QuantizedServerFrame = static_cast<uint32>(static_cast<int64>(Source.ServerFrame) + 1);
}

// Dequantize: recover ServerFrame from the unsigned stored value.
void FNetworkPhysicsPayloadNetSerializer::Dequantize(FNetSerializationContext&, const FNetDequantizeArgs& Args)
{
	const QuantizedType& Source = *reinterpret_cast<const QuantizedType*>(Args.Source);
	SourceType& Target = *reinterpret_cast<SourceType*>(Args.Target);

	Target.ServerFrame = static_cast<int32>(static_cast<int64>(Source.QuantizedServerFrame) - 1);
}

bool FNetworkPhysicsPayloadNetSerializer::IsEqual(FNetSerializationContext&, const FNetIsEqualArgs& Args)
{
	if (Args.bStateIsQuantized)
	{
		const QuantizedType& Value0 = *reinterpret_cast<const QuantizedType*>(Args.Source0);
		const QuantizedType& Value1 = *reinterpret_cast<const QuantizedType*>(Args.Source1);
		return Value0.QuantizedServerFrame == Value1.QuantizedServerFrame;
	}
	else
	{
		const SourceType& Value0 = *reinterpret_cast<const SourceType*>(Args.Source0);
		const SourceType& Value1 = *reinterpret_cast<const SourceType*>(Args.Source1);
		return Value0.ServerFrame == Value1.ServerFrame;
	}
}

bool FNetworkPhysicsPayloadNetSerializer::Validate(FNetSerializationContext&, const FNetValidateArgs&)
{
	return true;
}

// Registration: binds struct name "NetworkPhysicsPayload" to this serializer.
UE_NET_IMPLEMENT_FORWARDING_NETSERIALIZER_AND_REGISTRY_DELEGATES(NetworkPhysicsPayload, FNetworkPhysicsPayloadNetSerializer);


// ============================================================
// FNetworkPhysicsActionPayloadNetSerializer
// ============================================================

struct FNetworkPhysicsActionPayloadNetSerializer
{
	static constexpr uint32 Version = 0;

	typedef FNetworkPhysicsActionPayload SourceType;
	typedef FNetworkPhysicsActionPayloadQuantized QuantizedType;
	typedef FNetworkPhysicsActionPayloadNetSerializerConfig ConfigType;

	inline static const ConfigType DefaultConfig;

	static void Serialize(FNetSerializationContext&, const FNetSerializeArgs&);
	static void Deserialize(FNetSerializationContext&, const FNetDeserializeArgs&);

	static void SerializeDelta(FNetSerializationContext&, const FNetSerializeDeltaArgs&);
	static void DeserializeDelta(FNetSerializationContext&, const FNetDeserializeDeltaArgs&);

	static void Quantize(FNetSerializationContext&, const FNetQuantizeArgs&);
	static void Dequantize(FNetSerializationContext&, const FNetDequantizeArgs&);

	static bool IsEqual(FNetSerializationContext&, const FNetIsEqualArgs&);
	static bool Validate(FNetSerializationContext&, const FNetValidateArgs&);

};

UE_NET_IMPLEMENT_SERIALIZER(FNetworkPhysicsActionPayloadNetSerializer);

void FNetworkPhysicsActionPayloadNetSerializer::Serialize(FNetSerializationContext& Context, const FNetSerializeArgs& Args)
{
	const QuantizedType& Value = *reinterpret_cast<const QuantizedType*>(Args.Source);
	FNetBitStreamWriter* Writer = Context.GetBitStreamWriter();

	{
		UE_NET_TRACE_SCOPE(ServerFrame, *Writer, Context.GetTraceCollector(), ENetTraceVerbosity::Verbose);
		WritePackedUint32(Context, Args, Value.QuantizedServerFrame);
	}

	Writer->WriteBits(Value.SourceId, 32u);
	Writer->WriteBits(Value.ActionId, 16u);
}

void FNetworkPhysicsActionPayloadNetSerializer::Deserialize(FNetSerializationContext& Context, const FNetDeserializeArgs& Args)
{
	QuantizedType& Value = *reinterpret_cast<QuantizedType*>(Args.Target);

	Value.QuantizedServerFrame = ReadPackedUint32(Context, Args);

	FNetBitStreamReader* Reader = Context.GetBitStreamReader();
	Value.SourceId = Reader->ReadBits(32u);
	Value.ActionId = static_cast<uint16>(Reader->ReadBits(16u));
}

void FNetworkPhysicsActionPayloadNetSerializer::SerializeDelta(FNetSerializationContext& Context, const FNetSerializeDeltaArgs& Args)
{
	const QuantizedType& Value = *reinterpret_cast<const QuantizedType*>(Args.Source);
	const QuantizedType& PrevValue = *reinterpret_cast<const QuantizedType*>(Args.Prev);

	FNetBitStreamWriter* Writer = Context.GetBitStreamWriter();
	UE_NET_TRACE_SCOPE(ServerFrame, *Writer, Context.GetTraceCollector(), ENetTraceVerbosity::Verbose);

	// Delta-encode ServerFrame (1 bit if consecutive)
	const bool bIncrementalFrame = (Value.QuantizedServerFrame == PrevValue.QuantizedServerFrame + 1u);
	Writer->WriteBits(bIncrementalFrame, 1u);
	if (!bIncrementalFrame)
	{
		const int64 Delta = static_cast<int64>(Value.QuantizedServerFrame) - static_cast<int64>(PrevValue.QuantizedServerFrame);
		const bool bNegative = Delta < 0;
		Writer->WriteBits(bNegative, 1u);
		WritePackedUint32(Context, Args, static_cast<uint32>(FMath::Abs(Delta)));
	}

	// SourceId and ActionId: write only if changed
	const bool bSourceIdChanged = (Value.SourceId != PrevValue.SourceId);
	Writer->WriteBits(bSourceIdChanged, 1u);
	if (bSourceIdChanged)
	{
		Writer->WriteBits(Value.SourceId, 32u);
	}

	const bool bActionIdChanged = (Value.ActionId != PrevValue.ActionId);
	Writer->WriteBits(bActionIdChanged, 1u);
	if (bActionIdChanged)
	{
		Writer->WriteBits(Value.ActionId, 16u);
	}
}

void FNetworkPhysicsActionPayloadNetSerializer::DeserializeDelta(FNetSerializationContext& Context, const FNetDeserializeDeltaArgs& Args)
{
	QuantizedType& Value = *reinterpret_cast<QuantizedType*>(Args.Target);
	const QuantizedType& PrevValue = *reinterpret_cast<const QuantizedType*>(Args.Prev);

	FNetBitStreamReader* Reader = Context.GetBitStreamReader();

	const bool bIncrementalFrame = (Reader->ReadBits(1u) != 0u);
	if (bIncrementalFrame)
	{
		Value.QuantizedServerFrame = PrevValue.QuantizedServerFrame + 1u;
	}
	else
	{
		const bool bNegative = (Reader->ReadBits(1u) != 0u);
		const uint32 AbsDelta = ReadPackedUint32(Context, Args);
		const int64 SignedDelta = bNegative ? -static_cast<int64>(AbsDelta) : static_cast<int64>(AbsDelta);
		Value.QuantizedServerFrame = static_cast<uint32>(static_cast<int64>(PrevValue.QuantizedServerFrame) + SignedDelta);
	}

	const bool bSourceIdChanged = (Reader->ReadBits(1u) != 0u);
	Value.SourceId = bSourceIdChanged ? Reader->ReadBits(32u) : PrevValue.SourceId;

	const bool bActionIdChanged = (Reader->ReadBits(1u) != 0u);
	Value.ActionId = bActionIdChanged ? static_cast<uint16>(Reader->ReadBits(16u)) : PrevValue.ActionId;
}

void FNetworkPhysicsActionPayloadNetSerializer::Quantize(FNetSerializationContext&, const FNetQuantizeArgs& Args)
{
	const SourceType& Source = *reinterpret_cast<const SourceType*>(Args.Source);
	QuantizedType& Target = *reinterpret_cast<QuantizedType*>(Args.Target);

	Target.QuantizedServerFrame = static_cast<uint32>(static_cast<int64>(Source.ServerFrame) + 1);
	Target.SourceId = Source.SourceId;
	Target.ActionId = Source.ActionId;
}

void FNetworkPhysicsActionPayloadNetSerializer::Dequantize(FNetSerializationContext&, const FNetDequantizeArgs& Args)
{
	const QuantizedType& Source = *reinterpret_cast<const QuantizedType*>(Args.Source);
	SourceType& Target = *reinterpret_cast<SourceType*>(Args.Target);

	Target.ServerFrame = static_cast<int32>(static_cast<int64>(Source.QuantizedServerFrame) - 1);
	Target.SourceId = Source.SourceId;
	Target.ActionId = Source.ActionId;
}

bool FNetworkPhysicsActionPayloadNetSerializer::IsEqual(FNetSerializationContext&, const FNetIsEqualArgs& Args)
{
	if (Args.bStateIsQuantized)
	{
		const QuantizedType& V0 = *reinterpret_cast<const QuantizedType*>(Args.Source0);
		const QuantizedType& V1 = *reinterpret_cast<const QuantizedType*>(Args.Source1);
		return V0.QuantizedServerFrame == V1.QuantizedServerFrame
			&& V0.SourceId == V1.SourceId
			&& V0.ActionId == V1.ActionId;
	}
	else
	{
		const SourceType& V0 = *reinterpret_cast<const SourceType*>(Args.Source0);
		const SourceType& V1 = *reinterpret_cast<const SourceType*>(Args.Source1);
		return V0.ServerFrame == V1.ServerFrame
			&& V0.SourceId == V1.SourceId
			&& V0.ActionId == V1.ActionId;
	}
}

bool FNetworkPhysicsActionPayloadNetSerializer::Validate(FNetSerializationContext&, const FNetValidateArgs&)
{
	return true;
}

UE_NET_IMPLEMENT_FORWARDING_NETSERIALIZER_AND_REGISTRY_DELEGATES(NetworkPhysicsActionPayload, FNetworkPhysicsActionPayloadNetSerializer);


// ============================================================
// FNetworkPhysicsDataCollectionNetSerializer
// ============================================================

struct FNetworkPhysicsDataCollectionNetSerializer
{
	static constexpr uint32 Version = 0;

	static constexpr bool bHasDynamicState = true;
	static constexpr bool bIsForwardingSerializer = true;
	static constexpr bool bHasCustomNetReference = true;

	typedef FNetworkPhysicsDataCollection SourceType;
	typedef FNetworkPhysicsDataCollectionQuantized QuantizedType;
	typedef FNetworkPhysicsDataCollectionNetSerializerConfig ConfigType;

	inline static const ConfigType DefaultConfig;

	static void Serialize(FNetSerializationContext&, const FNetSerializeArgs&);
	static void Deserialize(FNetSerializationContext&, const FNetDeserializeArgs&);

	static void SerializeDelta(FNetSerializationContext&, const FNetSerializeDeltaArgs&);
	static void DeserializeDelta(FNetSerializationContext&, const FNetDeserializeDeltaArgs&);

	static void Quantize(FNetSerializationContext&, const FNetQuantizeArgs&);
	static void Dequantize(FNetSerializationContext&, const FNetDequantizeArgs&);

	static bool IsEqual(FNetSerializationContext&, const FNetIsEqualArgs&);
	static bool Validate(FNetSerializationContext&, const FNetValidateArgs&);

	static void CloneDynamicState(FNetSerializationContext&, const FNetCloneDynamicStateArgs&);
	static void FreeDynamicState(FNetSerializationContext&, const FNetFreeDynamicStateArgs&);
	static void CollectNetReferences(FNetSerializationContext&, const FNetCollectReferencesArgs&);

private:
	// Serialize one element's struct data with delta compression.
	// Uses Prev as the delta source if available and type-matching, otherwise a zero-initialized default.
	// Type identification is handled at the collection level - this only writes bValid + struct data.
	// SharedZeroStorage is grown in place per element; the owning top-level function frees it once at scope end.
	static void SerializeElementDelta(FNetSerializationContext& Context, const FNetSerializeDeltaArgs& BaseArgs,
		const FNetworkPhysicsElementQuantized& Curr, const FNetworkPhysicsElementQuantized* Prev,
		FNetSerializerAlignedStorage& SharedZeroStorage,
		FNetworkPhysicsDescriptorCache& Cache);

	// Deserialize one element's struct data with delta decompression (mirrors SerializeElementDelta).
	static void DeserializeElementDelta(FNetSerializationContext& Context, const FNetDeserializeDeltaArgs& BaseArgs,
		FNetworkPhysicsElementQuantized& Target, const FNetworkPhysicsElementQuantized* Prev,
		FNetSerializerAlignedStorage& SharedZeroStorage,
		FNetworkPhysicsDescriptorCache& Cache);

	// Set up an element's storage for a given struct type if it doesn't match already.
	static void EnsureElementType(FNetSerializationContext& Context, FNetworkPhysicsElementQuantized& Element,
		FName TypeName, const UScriptStruct* Struct, FNetworkPhysicsDescriptorCache& Cache);

	// Thread-safe descriptor/serializer cache, shared across all instances of this serializer.
	inline static FNetworkPhysicsDescriptorCache DescriptorCache;

};

UE_NET_IMPLEMENT_SERIALIZER(FNetworkPhysicsDataCollectionNetSerializer);

// ----- Element-Level Helpers -----

// Ensure an element's quantized storage matches the expected struct type.
// If the type changed, frees old storage and allocates new storage sized for the new type.
void FNetworkPhysicsDataCollectionNetSerializer::EnsureElementType(
	FNetSerializationContext& Context, FNetworkPhysicsElementQuantized& Element,
	FName TypeName, const UScriptStruct* Struct, FNetworkPhysicsDescriptorCache& Cache)
{
	// Already the correct type, nothing to do
	if (TypeName == Element.StructName)
	{
		return;
	}

	// Release old storage before re-allocating for the new type
	FreeElement(Context, Element, Cache);

	// Check if the struct type has a registered custom Iris serializer (e.g. FNetworkPhysicsPayloadNetSerializer)
	const FNetSerializer* CustomSerializer = Cache.FindCustomSerializer(TypeName);
	if (CustomSerializer)
	{
		// Custom serializer defines its own quantized size/alignment
		Element.StructData.AdjustSize(Context, CustomSerializer->QuantizedTypeSize, CustomSerializer->QuantizedTypeAlignment);
		Element.StructDescriptorTraits = EnumHasAnyFlags(CustomSerializer->Traits, ENetSerializerTraits::HasDynamicState)
			? EReplicationStateTraits::HasDynamicState : EReplicationStateTraits::None;
	}
	else
	{
		// No custom serializer - build a replication descriptor from UStruct reflection and use its sizing
		TRefCountPtr<const FReplicationStateDescriptor> Descriptor = Cache.FindOrAddDescriptor(Struct);
		if (ensureMsgf(Descriptor.IsValid(), TEXT("FNetworkPhysicsDataCollectionNetSerializer: failed to build descriptor for %s"), ToCStr(TypeName.ToString())))
		{
			Element.StructData.AdjustSize(Context, Descriptor->InternalSize, Descriptor->InternalAlignment);
			Element.StructDescriptorTraits = Descriptor->Traits;
		}
	}
	Element.StructName = TypeName;
}

// Serialize one element's data to the bit stream.
// Writes: [1 bit bValid] [struct data (always delta-compressed)]
// The concrete struct type is NOT written here - it is written once at the collection level.
// Delta source: the previous element if available and type-matching, otherwise a zero-initialized default.
// This avoids full serialization entirely - Iris's baseline acks rarely arrive in time for per-frame
// physics data, so always using delta (even against zero) gives better compression for element[0].
void FNetworkPhysicsDataCollectionNetSerializer::SerializeElementDelta(
	FNetSerializationContext& Context,
	const FNetSerializeDeltaArgs& BaseArgs,
	const FNetworkPhysicsElementQuantized& Curr,
	const FNetworkPhysicsElementQuantized* Prev,
	FNetSerializerAlignedStorage& SharedZeroStorage,
	FNetworkPhysicsDescriptorCache& Cache)
{
	FNetBitStreamWriter* Writer = Context.GetBitStreamWriter();

	// Write whether this element slot has valid data
	const bool bValid = !Curr.StructName.IsNone();
	Writer->WriteBool(bValid);
	if (!bValid)
	{
		return;
	}

	// Determine delta source: real previous element or zero-initialized default.
	// Both sender and receiver evaluate this condition independently (no wire bit needed).
	const bool bHasRealPrev = Prev && !Prev->StructName.IsNone()
		&& Prev->StructData.Num() > 0 && (Curr.StructName == Prev->StructName);

	// Resolve the serializer for this struct type (custom if registered, otherwise FStructNetSerializer)
	const FNetSerializer* CustomSerializer = Cache.FindCustomSerializer(Curr.StructName);

	// When no real previous element is available, grow the shared zero-buffer in place.
	// AdjustSize zero-fills via FMemory::Memzero - Iris guarantees "zero constructed state is valid".
	// Lifetime is owned by the calling top-level function via ON_SCOPE_EXIT.
	if (!bHasRealPrev)
	{
		if (CustomSerializer)
		{
			SharedZeroStorage.AdjustSize(Context, CustomSerializer->QuantizedTypeSize, CustomSerializer->QuantizedTypeAlignment);
		}
		else
		{
			TRefCountPtr<const FReplicationStateDescriptor> Descriptor = Cache.FindDescriptor(Curr.StructName);
			if (!Descriptor.IsValid())
			{
				Context.SetError(GNetError_InvalidValue);
				return;
			}
			SharedZeroStorage.AdjustSize(Context, Descriptor->InternalSize, Descriptor->InternalAlignment);
		}
	}

	const uint8* PrevData = bHasRealPrev ? Prev->StructData.GetData() : SharedZeroStorage.GetData();

	// Always delta-serialize
	if (CustomSerializer)
	{
		FNetSerializeDeltaArgs DeltaArgs = BaseArgs;
		DeltaArgs.Source = NetSerializerValuePointer(Curr.StructData.GetData());
		DeltaArgs.Prev = NetSerializerValuePointer(PrevData);
		DeltaArgs.NetSerializerConfig = CustomSerializer->DefaultConfig;
		CustomSerializer->SerializeDelta(Context, DeltaArgs);
	}
	else
	{
		FStructNetSerializerConfig StructConfig;
		StructConfig.StateDescriptor = Cache.FindDescriptor(Curr.StructName);
		if (!StructConfig.StateDescriptor.IsValid())
		{
			Context.SetError(GNetError_InvalidValue);
			return;
		}
		FNetSerializeDeltaArgs DeltaArgs = BaseArgs;
		DeltaArgs.Source = NetSerializerValuePointer(Curr.StructData.GetData());
		DeltaArgs.Prev = NetSerializerValuePointer(PrevData);
		DeltaArgs.NetSerializerConfig = NetSerializerConfigParam(&StructConfig);
		UE_NET_GET_SERIALIZER(FStructNetSerializer).SerializeDelta(Context, DeltaArgs);
	}
}

// Deserialize one element's data from the bit stream.
// Reads: [1 bit bValid] [struct data (always delta-compressed)]
// The element's type and storage must already be set up by the collection-level Deserialize
// before this function is called (via EnsureElementType).
// Delta source mirrors the sender: real previous element if available and type-matching, otherwise zero.
void FNetworkPhysicsDataCollectionNetSerializer::DeserializeElementDelta(
	FNetSerializationContext& Context,
	const FNetDeserializeDeltaArgs& BaseArgs,
	FNetworkPhysicsElementQuantized& Target,
	const FNetworkPhysicsElementQuantized* Prev,
	FNetSerializerAlignedStorage& SharedZeroStorage,
	FNetworkPhysicsDescriptorCache& Cache)
{
	FNetBitStreamReader* Reader = Context.GetBitStreamReader();
	const bool bValid = Reader->ReadBool();
	if (!bValid)
	{
		FreeElement(Context, Target, Cache);
		Target.StructName = FName();
		return;
	}

	if (Target.StructData.Num() == 0 || Target.StructName.IsNone())
	{
		Context.SetError(GNetError_InvalidValue);
		return;
	}

	// Determine delta source: must match sender's evaluation exactly.
	const bool bHasRealPrev = Prev && !Prev->StructName.IsNone()
		&& Prev->StructData.Num() > 0 && (Target.StructName == Prev->StructName);

	const FNetSerializer* CustomSerializer = Cache.FindCustomSerializer(Target.StructName);

	// When no real previous element is available, grow the shared zero-buffer in place.
	// Lifetime is owned by the calling top-level function via ON_SCOPE_EXIT.
	if (!bHasRealPrev)
	{
		if (CustomSerializer)
		{
			SharedZeroStorage.AdjustSize(Context, CustomSerializer->QuantizedTypeSize, CustomSerializer->QuantizedTypeAlignment);
		}
		else
		{
			TRefCountPtr<const FReplicationStateDescriptor> Descriptor = Cache.FindDescriptor(Target.StructName);
			if (!Descriptor.IsValid())
			{
				Context.SetError(GNetError_InvalidValue);
				return;
			}
			SharedZeroStorage.AdjustSize(Context, Descriptor->InternalSize, Descriptor->InternalAlignment);
		}
	}

	const uint8* PrevData = bHasRealPrev ? Prev->StructData.GetData() : SharedZeroStorage.GetData();

	// Always delta-deserialize
	if (CustomSerializer)
	{
		FNetDeserializeDeltaArgs DeltaArgs = BaseArgs;
		DeltaArgs.Target = NetSerializerValuePointer(Target.StructData.GetData());
		DeltaArgs.Prev = NetSerializerValuePointer(PrevData);
		DeltaArgs.NetSerializerConfig = CustomSerializer->DefaultConfig;
		CustomSerializer->DeserializeDelta(Context, DeltaArgs);
	}
	else
	{
		FStructNetSerializerConfig StructConfig;
		StructConfig.StateDescriptor = Cache.FindDescriptor(Target.StructName);
		if (!StructConfig.StateDescriptor.IsValid())
		{
			Context.SetError(GNetError_InvalidValue);
			return;
		}
		FNetDeserializeDeltaArgs DeltaArgs = BaseArgs;
		DeltaArgs.Target = NetSerializerValuePointer(Target.StructData.GetData());
		DeltaArgs.Prev = NetSerializerValuePointer(PrevData);
		DeltaArgs.NetSerializerConfig = NetSerializerConfigParam(&StructConfig);
		UE_NET_GET_SERIALIZER(FStructNetSerializer).DeserializeDelta(Context, DeltaArgs);
	}
}

// ----- Top-Level Collection Functions -----
// Iris NetSerializer pipeline: Quantize -> Serialize/SerializeDelta -> (wire) -> Deserialize/DeserializeDelta -> Dequantize
// Quantize: converts game-thread source data (FNetworkPhysicsDataCollection) into quantized form.
// Serialize: writes the quantized form to the bit stream (called when no Iris baseline is available).
// SerializeDelta: writes the quantized form as a delta against an Iris-managed previous state.
// Deserialize/DeserializeDelta: inverse of Serialize/SerializeDelta - reads from bit stream into quantized form.
// Dequantize: converts quantized form back into game-thread data.

// --- Quantize: game-thread data -> quantized form ---
// Converts each TInstancedStruct element into a type-tagged quantized blob.
void FNetworkPhysicsDataCollectionNetSerializer::Quantize(FNetSerializationContext& Context, const FNetQuantizeArgs& Args)
{
	const SourceType& Source = *reinterpret_cast<const SourceType*>(Args.Source);
	QuantizedType& Target = *reinterpret_cast<QuantizedType*>(Args.Target);

	const int32 NewCount = Source.DataArray.Num();
	const int32 OldCount = static_cast<int32>(Target.Elements.Num());

	// Shrinking: free quantized elements that no longer have source data
	for (int32 Idx = NewCount; Idx < OldCount; ++Idx)
	{
		FreeElement(Context, Target.Elements.GetData()[Idx], DescriptorCache);
	}
	Target.Elements.AdjustSize(Context, static_cast<uint32>(NewCount));

	// Memoize descriptor + custom serializer across elements so consecutive same-type elements pay
	// one FindOrAddDescriptor / FindCustomSerializer / GetCachedPathName per unique type, not per element.
	// Keyed on UScriptStruct* (singleton per type) so same-type runs skip the path-name TMap lookup entirely.
	const UScriptStruct* LastStruct = nullptr;
	FName LastStructName;
	TRefCountPtr<const FReplicationStateDescriptor> LastDescriptor;
	const FNetSerializer* LastCustomSerializer = nullptr;

	FNetworkPhysicsElementQuantized* Elements = Target.Elements.GetData();
	for (int32 Idx = 0; Idx < NewCount; ++Idx)
	{
		FNetworkPhysicsElementQuantized& El = Elements[Idx];
		const TInstancedStruct<FNetworkPhysicsPayload>& SrcEl = Source.DataArray[Idx];

		// Empty source element - clear the quantized slot
		if (!SrcEl.IsValid())
		{
			FreeElement(Context, El, DescriptorCache);
			El.StructName = FName();
			continue;
		}

		// Identify the concrete struct type (e.g. FNetworkPawnInputIris) and resolve descriptor /
		// custom serializer / path-name only when the struct pointer changes between elements.
		const UScriptStruct* Struct = SrcEl.GetScriptStruct();
		if (Struct != LastStruct)
		{
			LastStruct = Struct;
			LastStructName = DescriptorCache.GetCachedPathName(Struct);
			LastCustomSerializer = DescriptorCache.FindCustomSerializer(LastStructName);
			LastDescriptor = LastCustomSerializer ? TRefCountPtr<const FReplicationStateDescriptor>{} : DescriptorCache.FindOrAddDescriptor(Struct);
		}
		const FName& PathFName = LastStructName;

		// If the type changed from what this slot previously held, re-allocate storage
		if (PathFName != El.StructName)
		{
			FreeElement(Context, El, DescriptorCache);

			if (LastCustomSerializer)
			{
				El.StructData.AdjustSize(Context, LastCustomSerializer->QuantizedTypeSize, LastCustomSerializer->QuantizedTypeAlignment);
				El.StructDescriptorTraits = EnumHasAnyFlags(LastCustomSerializer->Traits, ENetSerializerTraits::HasDynamicState)
					? EReplicationStateTraits::HasDynamicState : EReplicationStateTraits::None;
			}
			else
			{
				checkf(LastDescriptor.IsValid(), TEXT("FNetworkPhysicsDataCollectionNetSerializer::Quantize: failed to build descriptor for %s"), ToCStr(PathFName.ToString()));
				El.StructData.AdjustSize(Context, LastDescriptor->InternalSize, LastDescriptor->InternalAlignment);
				El.StructDescriptorTraits = LastDescriptor->Traits;
			}

			El.StructName = PathFName;
		}

		if (LastCustomSerializer)
		{
			FNetQuantizeArgs StructArgs = Args;
			StructArgs.Source = NetSerializerValuePointer(SrcEl.GetMemory());
			StructArgs.Target = NetSerializerValuePointer(El.StructData.GetData());
			StructArgs.NetSerializerConfig = LastCustomSerializer->DefaultConfig;
			LastCustomSerializer->Quantize(Context, StructArgs);
		}
		else
		{
			FStructNetSerializerConfig StructConfig;
			StructConfig.StateDescriptor = LastDescriptor;
			FNetQuantizeArgs StructArgs = Args;
			StructArgs.Source = NetSerializerValuePointer(SrcEl.GetMemory());
			StructArgs.Target = NetSerializerValuePointer(El.StructData.GetData());
			StructArgs.NetSerializerConfig = NetSerializerConfigParam(&StructConfig);
			UE_NET_GET_SERIALIZER(FStructNetSerializer).Quantize(Context, StructArgs);
		}
	}
}

void FNetworkPhysicsDataCollectionNetSerializer::Dequantize(FNetSerializationContext& Context, const FNetDequantizeArgs& Args)
{
	const QuantizedType& Source = *reinterpret_cast<const QuantizedType*>(Args.Source);
	SourceType& Target = *reinterpret_cast<SourceType*>(Args.Target);

	const int32 Count = static_cast<int32>(Source.Elements.Num());
	Target.DataArray.SetNum(Count);

	const FNetworkPhysicsElementQuantized* Elements = Source.Elements.GetData();
	for (int32 Idx = 0; Idx < Count; ++Idx)
	{
		const FNetworkPhysicsElementQuantized& El = Elements[Idx];
		TInstancedStruct<FNetworkPhysicsPayload>& DstEl = Target.DataArray[Idx];

		if (El.StructName.IsNone())
		{
			DstEl.Reset();
			continue;
		}

		const UScriptStruct* Struct = DescriptorCache.FindOrCacheStruct(El.StructName);
		if (!ensureMsgf(Struct != nullptr, TEXT("FNetworkPhysicsDataCollectionNetSerializer::Dequantize: unknown struct type '%s'"), ToCStr(El.StructName.ToString())))
		{
			continue;
		}

		TRefCountPtr<const FReplicationStateDescriptor> Descriptor = DescriptorCache.FindOrAddDescriptor(Struct);
		if (!ensureMsgf(Descriptor.IsValid(), TEXT("FNetworkPhysicsDataCollectionNetSerializer::Dequantize: missing descriptor for %s"), ToCStr(El.StructName.ToString())))
		{
			continue;
		}

		if (Struct != DstEl.GetScriptStruct())
		{
			DstEl.InitializeAsScriptStruct(Struct);
		}

		// Dequantize using custom serializer if registered, otherwise fall back to FStructNetSerializer
		const FNetSerializer* CustomSerializer = DescriptorCache.FindCustomSerializer(El.StructName);
		if (CustomSerializer)
		{
			FNetDequantizeArgs StructArgs = Args;
			StructArgs.Source = NetSerializerValuePointer(El.StructData.GetData());
			StructArgs.Target = NetSerializerValuePointer(DstEl.GetMutableMemory());
			StructArgs.NetSerializerConfig = CustomSerializer->DefaultConfig;
			CustomSerializer->Dequantize(Context, StructArgs);
		}
		else
		{
			FStructNetSerializerConfig StructConfig;
			StructConfig.StateDescriptor = Descriptor;
			FNetDequantizeArgs StructArgs = Args;
			StructArgs.Source = NetSerializerValuePointer(El.StructData.GetData());
			StructArgs.Target = NetSerializerValuePointer(DstEl.GetMutableMemory());
			StructArgs.NetSerializerConfig = NetSerializerConfigParam(&StructConfig);
			UE_NET_GET_SERIALIZER(FStructNetSerializer).Dequantize(Context, StructArgs);
		}
	}
}

// --- Serialize: quantized form -> bit stream (full, no Iris baseline) ---
// Wire format: [packed count] [1 bit bHasValidType] [optional 32-bit type hash] [elements with intra-array delta]
// Called by Iris when no acknowledged baseline exists (first send, or baseline expired).
// Uses intra-array delta chaining so consecutive elements still get efficient encoding.
void FNetworkPhysicsDataCollectionNetSerializer::Serialize(FNetSerializationContext& Context, const FNetSerializeArgs& Args)
{
	const QuantizedType& Value = *reinterpret_cast<const QuantizedType*>(Args.Source);
	const uint32 Count = Value.Elements.Num();

	// Write element count as variable-length packed uint32
	FNetBitStreamWriter* Writer = Context.GetBitStreamWriter();
	WritePackedUint32(Context, Args, Count);

	if (Count == 0u)
	{
		return;
	}

	// Zero-baseline buffer shared across elements; grown in place by the element helper,
	// freed once here regardless of early-return via SetError.
	FNetSerializerAlignedStorage SharedZeroStorage;
	ON_SCOPE_EXIT{ SharedZeroStorage.Free(Context); };

	// Find the first valid element - invalid slots have StructName == None after Quantize clears them.
	const FNetworkPhysicsElementQuantized* Elements = Value.Elements.GetData();
	FName CollectionTypeName;
	for (uint32 Idx = 0u; Idx < Count; ++Idx)
	{
		if (!Elements[Idx].StructName.IsNone())
		{
			CollectionTypeName = Elements[Idx].StructName;
			break;
		}
	}

	// Write 1 bit indicating whether a valid type exists, then the 32-bit CRC hash if so.
	// All elements invalid is a representable state - the per-element bValid bits handle it.
	const bool bHasValidType = !CollectionTypeName.IsNone();
	Writer->WriteBool(bHasValidType);
	if (bHasValidType)
	{
		WriteStructNameHash(*Writer, CollectionTypeName, DescriptorCache);
	}

	// Serialize each element with intra-array delta chaining:
	// Element[0]: delta against zero-initialized default (no previous element available)
	// Element[i>0]: delta against element[i-1] -> consecutive ServerFrames compress to 1 bit
	FNetSerializeDeltaArgs DeltaArgs = {};
	DeltaArgs.NetSerializerConfig = Args.NetSerializerConfig;
	for (uint32 Idx = 0u; Idx < Count; ++Idx)
	{
		const FNetworkPhysicsElementQuantized* Prev = (Idx > 0u) ? &Elements[Idx - 1u] : nullptr;
		SerializeElementDelta(Context, DeltaArgs, Elements[Idx], Prev, SharedZeroStorage, DescriptorCache);
	}
}

// --- Deserialize: bit stream -> quantized form (full, no Iris baseline) ---
// Inverse of Serialize. Reads: [packed count] [1 bit bHasValidType] [optional 32-bit type hash] [elements with intra-array delta]
void FNetworkPhysicsDataCollectionNetSerializer::Deserialize(FNetSerializationContext& Context, const FNetDeserializeArgs& Args)
{
	QuantizedType& Value = *reinterpret_cast<QuantizedType*>(Args.Target);

	// Read element count
	const uint32 NewCount = ReadPackedUint32(Context, Args);

	// Shrinking: free elements that no longer exist, then resize
	const uint32 OldCount = Value.Elements.Num();
	for (uint32 Idx = NewCount; Idx < OldCount; ++Idx)
	{
		FreeElement(Context, Value.Elements.GetData()[Idx], DescriptorCache);
	}
	Value.Elements.AdjustSize(Context, NewCount);

	if (NewCount == 0u)
	{
		return;
	}

	// Read the concrete struct type hash and resolve to UScriptStruct (if a valid type was sent)
	FNetBitStreamReader* Reader = Context.GetBitStreamReader();
	const bool bHasValidType = Reader->ReadBool();
	FName CollectionTypeName;
	const UScriptStruct* CollectionStruct = nullptr;
	if (bHasValidType)
	{
		CollectionTypeName = ReadStructNameHash(*Reader, DescriptorCache);
		CollectionStruct = DescriptorCache.FindOrCacheStruct(CollectionTypeName);
		if (!CollectionStruct)
		{
			Context.SetError(GNetError_InvalidValue);
			return;
		}
	}

	// Pre-allocate storage for all elements using the resolved type (skipped if all elements are invalid)
	if (CollectionStruct)
	{
		FNetworkPhysicsElementQuantized* Elements = Value.Elements.GetData();
		for (uint32 Idx = 0u; Idx < NewCount; ++Idx)
		{
			EnsureElementType(Context, Elements[Idx], CollectionTypeName, CollectionStruct, DescriptorCache);
		}
	}

	// Shared zero-baseline buffer, lifetime owned at this scope.
	FNetSerializerAlignedStorage SharedZeroStorage;
	ON_SCOPE_EXIT{ SharedZeroStorage.Free(Context); };

	// Read each element with intra-array delta (mirrors the sender's chaining)
	FNetDeserializeDeltaArgs DeltaArgs = {};
	DeltaArgs.NetSerializerConfig = Args.NetSerializerConfig;
	FNetworkPhysicsElementQuantized* Elements = Value.Elements.GetData();
	for (uint32 Idx = 0u; Idx < NewCount; ++Idx)
	{
		const FNetworkPhysicsElementQuantized* Prev = (Idx > 0u) ? &Elements[Idx - 1u] : nullptr;
		DeserializeElementDelta(Context, DeltaArgs, Elements[Idx], Prev, SharedZeroStorage, DescriptorCache);
	}
}

// --- SerializeDelta: quantized form -> bit stream (delta against Iris-managed baseline) ---
// Wire format: [1 bit bSameCount] [optional count] [1 bit bTypeChanged] [optional type hash] [elements with delta chaining]
// Called by Iris when an acknowledged baseline exists to delta against.
void FNetworkPhysicsDataCollectionNetSerializer::SerializeDelta(FNetSerializationContext& Context, const FNetSerializeDeltaArgs& Args)
{
	const QuantizedType& Curr = *reinterpret_cast<const QuantizedType*>(Args.Source);
	const QuantizedType& IrisPrev = *reinterpret_cast<const QuantizedType*>(Args.Prev);

	const uint32 CurrCount = Curr.Elements.Num();
	const uint32 PrevCount = IrisPrev.Elements.Num();

	// Delta-encode the count: 1 bit if unchanged, otherwise write full count
	FNetBitStreamWriter* Writer = Context.GetBitStreamWriter();
	const bool bSameCount = (CurrCount == PrevCount);
	Writer->WriteBool(bSameCount);
	if (!bSameCount)
	{
		WritePackedUint32(Context, Args, CurrCount);
	}

	if (CurrCount == 0u)
	{
		return;
	}

	const FNetworkPhysicsElementQuantized* CurrElements = Curr.Elements.GetData();
	const FNetworkPhysicsElementQuantized* PrevElements = IrisPrev.Elements.GetData();

	// Delta-encode the collection type: 1 bit if unchanged, otherwise write 32-bit hash.
	// Find the first valid element in each array - invalid slots have StructName == None.
	FName CurrTypeName;
	for (uint32 Idx = 0u; Idx < CurrCount; ++Idx)
	{
		if (!CurrElements[Idx].StructName.IsNone())
		{
			CurrTypeName = CurrElements[Idx].StructName;
			break;
		}
	}
	FName PrevTypeName;
	for (uint32 Idx = 0u; Idx < PrevCount; ++Idx)
	{
		if (!PrevElements[Idx].StructName.IsNone())
		{
			PrevTypeName = PrevElements[Idx].StructName;
			break;
		}
	}
	const bool bTypeChanged = (CurrTypeName != PrevTypeName);
	Writer->WriteBool(bTypeChanged);
	if (bTypeChanged)
	{
		const bool bHasValidType = !CurrTypeName.IsNone();
		Writer->WriteBool(bHasValidType);
		if (bHasValidType)
		{
			WriteStructNameHash(*Writer, CurrTypeName, DescriptorCache);
		}
	}

	// Shared zero-baseline buffer, lifetime owned at this scope.
	FNetSerializerAlignedStorage SharedZeroStorage;
	ON_SCOPE_EXIT{ SharedZeroStorage.Free(Context); };

	// Intra-array delta chaining:
	// Element[0] -> delta against IrisPrev[0] (Iris-managed baseline from acknowledged state)
	// Element[i>0] -> delta against Curr[i-1] (preceding element in the current array)
	// This gives consecutive ServerFrames a 1-bit encoding even within the Iris delta path.
	auto GetDeltaSource = [&](const uint32 Idx) -> const FNetworkPhysicsElementQuantized*
	{
		if (Idx == 0u)
		{
			return (PrevCount > 0u && !bTypeChanged) ? &PrevElements[0] : nullptr;
		}
		return &CurrElements[Idx - 1u];
	};

	for (uint32 Idx = 0u; Idx < CurrCount; ++Idx)
	{
		SerializeElementDelta(Context, Args, CurrElements[Idx], GetDeltaSource(Idx), SharedZeroStorage, DescriptorCache);
	}
}

// --- DeserializeDelta: bit stream -> quantized form (delta against Iris-managed baseline) ---
// Inverse of SerializeDelta. Reads: [1 bit bSameCount] [optional count] [1 bit bTypeChanged] [optional hash] [elements]
void FNetworkPhysicsDataCollectionNetSerializer::DeserializeDelta(FNetSerializationContext& Context, const FNetDeserializeDeltaArgs& Args)
{
	QuantizedType& Target = *reinterpret_cast<QuantizedType*>(Args.Target);
	const QuantizedType& IrisPrev = *reinterpret_cast<const QuantizedType*>(Args.Prev);

	FNetBitStreamReader* Reader = Context.GetBitStreamReader();

	// Read count: if bSameCount bit is false, read the new count; otherwise reuse prev count
	uint32 NewCount = IrisPrev.Elements.Num();
	const bool bSameCount = Reader->ReadBool();
	if (!bSameCount)
	{
		NewCount = ReadPackedUint32(Context, Args);
	}

	// Free excess elements and resize the quantized array
	const uint32 OldCount = Target.Elements.Num();
	for (uint32 Idx = NewCount; Idx < OldCount; ++Idx)
	{
		FreeElement(Context, Target.Elements.GetData()[Idx], DescriptorCache);
	}
	Target.Elements.AdjustSize(Context, NewCount);

	if (NewCount == 0u)
	{
		return;
	}

	const FNetworkPhysicsElementQuantized* PrevElements = IrisPrev.Elements.GetData();
	const uint32 PrevCount = IrisPrev.Elements.Num();

	// Read collection-level type: if bTypeChanged, read bHasValidType + optional hash; otherwise infer from baseline
	const bool bTypeChanged = Reader->ReadBool();
	FName CollectionTypeName;
	const UScriptStruct* CollectionStruct = nullptr;
	if (bTypeChanged)
	{
		const bool bHasValidType = Reader->ReadBool();
		if (bHasValidType)
		{
			CollectionTypeName = ReadStructNameHash(*Reader, DescriptorCache);
			CollectionStruct = DescriptorCache.FindOrCacheStruct(CollectionTypeName);
			if (!CollectionStruct)
			{
				Context.SetError(GNetError_InvalidValue);
				return;
			}
		}
	}
	else if (PrevCount > 0u)
	{
		// Type unchanged - reuse from the baseline's first valid element
		for (uint32 Idx = 0u; Idx < PrevCount; ++Idx)
		{
			if (!PrevElements[Idx].StructName.IsNone())
			{
				CollectionTypeName = PrevElements[Idx].StructName;
				break;
			}
		}
		CollectionStruct = DescriptorCache.FindOrCacheStruct(CollectionTypeName);
	}

	// Pre-allocate storage for all elements using the resolved type
	if (!CollectionTypeName.IsNone() && CollectionStruct)
	{
		FNetworkPhysicsElementQuantized* Elements = Target.Elements.GetData();
		for (uint32 Idx = 0u; Idx < NewCount; ++Idx)
		{
			EnsureElementType(Context, Elements[Idx], CollectionTypeName, CollectionStruct, DescriptorCache);
		}
	}

	// Shared zero-baseline buffer, lifetime owned at this scope.
	FNetSerializerAlignedStorage SharedZeroStorage;
	ON_SCOPE_EXIT{ SharedZeroStorage.Free(Context); };

	// Read each element, mirroring the sender's delta chaining:
	// Element[0] -> delta against IrisPrev[0] (baseline)
	// Element[i>0] -> delta against just-decoded element[i-1]
	auto GetDeltaSource = [&](const uint32 Idx) -> const FNetworkPhysicsElementQuantized*
	{
		if (Idx == 0u)
		{
			return (PrevCount > 0u && !bTypeChanged) ? &PrevElements[0] : nullptr;
		}
		return &Target.Elements.GetData()[Idx - 1u];
	};

	FNetworkPhysicsElementQuantized* Elements = Target.Elements.GetData();
	for (uint32 Idx = 0u; Idx < NewCount; ++Idx)
	{
		DeserializeElementDelta(Context, Args, Elements[Idx], GetDeltaSource(Idx), SharedZeroStorage, DescriptorCache);
	}
}

bool FNetworkPhysicsDataCollectionNetSerializer::IsEqual(FNetSerializationContext&, const FNetIsEqualArgs& Args)
{
	if (IsEqualImpl<QuantizedType, FNetworkPhysicsElementQuantized>(Args))
	{
		return true;
	}
	if (!Args.bStateIsQuantized)
	{
		const SourceType& V0 = *reinterpret_cast<const SourceType*>(Args.Source0);
		const SourceType& V1 = *reinterpret_cast<const SourceType*>(Args.Source1);
		if (V0.DataArray.Num() != V1.DataArray.Num())
		{
			return false;
		}
		for (int32 Idx = 0; Idx < V0.DataArray.Num(); ++Idx)
		{
			if (V0.DataArray[Idx] != V1.DataArray[Idx])
			{
				return false;
			}
		}
		return true;
	}
	return false;
}

bool FNetworkPhysicsDataCollectionNetSerializer::Validate(FNetSerializationContext&, const FNetValidateArgs&)
{
	return true;
}

void FNetworkPhysicsDataCollectionNetSerializer::CloneDynamicState(FNetSerializationContext& Context, const FNetCloneDynamicStateArgs& Args)
{
	CloneDynamicStateImpl<QuantizedType, FNetworkPhysicsElementQuantized>(Context, Args, DescriptorCache);
}

void FNetworkPhysicsDataCollectionNetSerializer::FreeDynamicState(FNetSerializationContext& Context, const FNetFreeDynamicStateArgs& Args)
{
	FreeDynamicStateImpl<QuantizedType, FNetworkPhysicsElementQuantized>(Context, Args, DescriptorCache);
}

void FNetworkPhysicsDataCollectionNetSerializer::CollectNetReferences(FNetSerializationContext& Context, const FNetCollectReferencesArgs& Args)
{
	CollectNetReferencesImpl<QuantizedType, FNetworkPhysicsElementQuantized>(Context, Args, DescriptorCache);
}

UE_NET_IMPLEMENT_FORWARDING_NETSERIALIZER_AND_REGISTRY_DELEGATES(NetworkPhysicsDataCollection, FNetworkPhysicsDataCollectionNetSerializer);


// ============================================================
// FNetworkPhysicsActionCollectionNetSerializer
// ============================================================
// Mirrors FNetworkPhysicsDataCollectionNetSerializer with these differences:
// 1. SourceType / QuantizedType use FNetworkPhysicsActionPayload / FNetworkPhysicsActionElementQuantized.
// 2. Heterogeneous: elements can have different concrete types within the same array.
//	Same-type adjacent elements delta-chain; type changes delta against a zero-initialized default.
// 3. Per-element type flag: each element writes a 1-bit bSameType flag vs the collection-level type.
// 4. Element data always goes through FStructNetSerializer (no custom serializer dispatch).
//	This is intentional - action payload types do not register custom Iris serializers.

struct FNetworkPhysicsActionCollectionNetSerializer
{
	static constexpr uint32 Version = 0;

	static constexpr bool bHasDynamicState = true;
	static constexpr bool bIsForwardingSerializer = true;
	static constexpr bool bHasCustomNetReference = true;

	typedef FNetworkPhysicsActionCollection SourceType;
	typedef FNetworkPhysicsActionCollectionQuantized QuantizedType;
	typedef FNetworkPhysicsActionCollectionNetSerializerConfig ConfigType;

	inline static const ConfigType DefaultConfig;

	static void Serialize(FNetSerializationContext&, const FNetSerializeArgs&);
	static void Deserialize(FNetSerializationContext&, const FNetDeserializeArgs&);

	static void SerializeDelta(FNetSerializationContext&, const FNetSerializeDeltaArgs&);
	static void DeserializeDelta(FNetSerializationContext&, const FNetDeserializeDeltaArgs&);

	static void Quantize(FNetSerializationContext&, const FNetQuantizeArgs&);
	static void Dequantize(FNetSerializationContext&, const FNetDequantizeArgs&);

	static bool IsEqual(FNetSerializationContext&, const FNetIsEqualArgs&);
	static bool Validate(FNetSerializationContext&, const FNetValidateArgs&);

	static void CloneDynamicState(FNetSerializationContext&, const FNetCloneDynamicStateArgs&);
	static void FreeDynamicState(FNetSerializationContext&, const FNetFreeDynamicStateArgs&);
	static void CollectNetReferences(FNetSerializationContext&, const FNetCollectReferencesArgs&);

private:
	static void SerializeElementDelta(FNetSerializationContext&, const FNetSerializeDeltaArgs&,
		const FNetworkPhysicsActionElementQuantized& Curr,
		const FNetworkPhysicsActionElementQuantized* Prev,
		FName CollectionTypeName,
		FNetSerializerAlignedStorage& SharedZeroStorage,
		FNetworkPhysicsDescriptorCache&);
	static void DeserializeElementDelta(FNetSerializationContext&, const FNetDeserializeDeltaArgs&,
		FNetworkPhysicsActionElementQuantized& Target,
		const FNetworkPhysicsActionElementQuantized* Prev,
		FName CollectionTypeName,
		FNetSerializerAlignedStorage& SharedZeroStorage,
		FNetworkPhysicsDescriptorCache&);
	static void EnsureElementType(FNetSerializationContext&, FNetworkPhysicsActionElementQuantized&,
		FName TypeName, const UScriptStruct*, FNetworkPhysicsDescriptorCache&);

	// Thread-safe descriptor/serializer cache, shared across all instances of this serializer.
	inline static FNetworkPhysicsDescriptorCache DescriptorCache;

};

UE_NET_IMPLEMENT_SERIALIZER(FNetworkPhysicsActionCollectionNetSerializer);

// ----- action element helpers -----

void FNetworkPhysicsActionCollectionNetSerializer::EnsureElementType(
	FNetSerializationContext& Context, FNetworkPhysicsActionElementQuantized& Element,
	FName TypeName, const UScriptStruct* Struct, FNetworkPhysicsDescriptorCache& Cache)
{
	if (TypeName == Element.StructName)
	{
		return;
	}
	FreeElement(Context, Element, Cache);
	TRefCountPtr<const FReplicationStateDescriptor> Descriptor = Cache.FindOrAddDescriptor(Struct);
	if (ensureMsgf(Descriptor.IsValid(), TEXT("FNetworkPhysicsActionCollectionNetSerializer: failed to build descriptor for %s"), ToCStr(TypeName.ToString())))
	{
		Element.StructData.AdjustSize(Context, Descriptor->InternalSize, Descriptor->InternalAlignment);
		Element.StructDescriptorTraits = Descriptor->Traits;
	}
	Element.StructName = TypeName;
}

// Serialize one action element: bValid + bSameType flag + optional per-element type name + struct data (always delta).
// CollectionTypeName is the "common type" written at the collection level.
// Delta source: the previous element if available and type-matching, otherwise a zero-initialized default.
// This covers both element[0] and type-boundary elements in heterogeneous arrays.
void FNetworkPhysicsActionCollectionNetSerializer::SerializeElementDelta(
	FNetSerializationContext& Context, const FNetSerializeDeltaArgs& BaseArgs,
	const FNetworkPhysicsActionElementQuantized& Curr,
	const FNetworkPhysicsActionElementQuantized* Prev,
	FName CollectionTypeName,
	FNetSerializerAlignedStorage& SharedZeroStorage,
	FNetworkPhysicsDescriptorCache& Cache)
{
	FNetBitStreamWriter* Writer = Context.GetBitStreamWriter();
	Writer->WriteBool(!Curr.StructName.IsNone());
	if (Curr.StructName.IsNone())
	{
		return;
	}

	// 1 bit: does this element match the collection-level type?
	const bool bSameType = (Curr.StructName == CollectionTypeName);
	Writer->WriteBool(bSameType);
	if (!bSameType)
	{
		WriteStructNameHash(*Writer, Curr.StructName, Cache);
	}

	FStructNetSerializerConfig StructConfig;
	StructConfig.StateDescriptor = Cache.FindDescriptor(Curr.StructName);
	if (!StructConfig.StateDescriptor.IsValid())
	{
		Context.SetError(GNetError_InvalidValue);
		return;
	}

	// Determine delta source: real previous element or zero-initialized default.
	const bool bHasRealPrev = Prev && !Prev->StructName.IsNone()
		&& Prev->StructData.Num() > 0 && (Curr.StructName == Prev->StructName);

	if (!bHasRealPrev)
	{
		SharedZeroStorage.AdjustSize(Context, StructConfig.StateDescriptor->InternalSize, StructConfig.StateDescriptor->InternalAlignment);
	}

	const uint8* PrevData = bHasRealPrev ? Prev->StructData.GetData() : SharedZeroStorage.GetData();

	// Always delta-serialize
	FNetSerializeDeltaArgs DeltaArgs = BaseArgs;
	DeltaArgs.Source = NetSerializerValuePointer(Curr.StructData.GetData());
	DeltaArgs.Prev = NetSerializerValuePointer(PrevData);
	DeltaArgs.NetSerializerConfig = NetSerializerConfigParam(&StructConfig);
	UE_NET_GET_SERIALIZER(FStructNetSerializer).SerializeDelta(Context, DeltaArgs);
}

// Deserialize one action element: bValid + bSameType flag + optional per-element type name + struct data (always delta).
// Delta source mirrors the sender: real previous element if available and type-matching, otherwise zero.
void FNetworkPhysicsActionCollectionNetSerializer::DeserializeElementDelta(
	FNetSerializationContext& Context, const FNetDeserializeDeltaArgs& BaseArgs,
	FNetworkPhysicsActionElementQuantized& Target,
	const FNetworkPhysicsActionElementQuantized* Prev,
	FName CollectionTypeName,
	FNetSerializerAlignedStorage& SharedZeroStorage,
	FNetworkPhysicsDescriptorCache& Cache)
{
	FNetBitStreamReader* Reader = Context.GetBitStreamReader();
	const bool bValid = Reader->ReadBool();
	if (!bValid)
	{
		FreeElement(Context, Target, Cache);
		Target.StructName = FName();
		return;
	}

	// Resolve the element's type.
	FName ElementTypeName;
	const bool bSameType = Reader->ReadBool();
	if (bSameType)
	{
		ElementTypeName = CollectionTypeName;
	}
	else
	{
		ElementTypeName = ReadStructNameHash(*Reader, DescriptorCache);
	}

	const UScriptStruct* Struct = FindObject<UScriptStruct>(nullptr, *ElementTypeName.ToString());
	if (!ensureMsgf(Struct, TEXT("Invalid action struct type '%s'"), ToCStr(ElementTypeName.ToString())))
	{
		Context.SetError(GNetError_InvalidValue);
		return;
	}

	EnsureElementType(Context, Target, ElementTypeName, Struct, Cache);

	FStructNetSerializerConfig StructConfig;
	StructConfig.StateDescriptor = Cache.FindOrAddDescriptor(Struct);
	if (!StructConfig.StateDescriptor.IsValid())
	{
		Context.SetError(GNetError_InvalidValue);
		return;
	}

	// Determine delta source: must match sender's evaluation exactly.
	const bool bHasRealPrev = Prev && !Prev->StructName.IsNone()
		&& Prev->StructData.Num() > 0 && (Target.StructName == Prev->StructName);

	if (!bHasRealPrev)
	{
		SharedZeroStorage.AdjustSize(Context, StructConfig.StateDescriptor->InternalSize, StructConfig.StateDescriptor->InternalAlignment);
	}

	const uint8* PrevData = bHasRealPrev ? Prev->StructData.GetData() : SharedZeroStorage.GetData();

	// Always delta-deserialize
	FNetDeserializeDeltaArgs DeltaArgs = BaseArgs;
	DeltaArgs.Target = NetSerializerValuePointer(Target.StructData.GetData());
	DeltaArgs.Prev = NetSerializerValuePointer(PrevData);
	DeltaArgs.NetSerializerConfig = NetSerializerConfigParam(&StructConfig);
	UE_NET_GET_SERIALIZER(FStructNetSerializer).DeserializeDelta(Context, DeltaArgs);
}

// ----- top-level functions -----

void FNetworkPhysicsActionCollectionNetSerializer::Quantize(FNetSerializationContext& Context, const FNetQuantizeArgs& Args)
{
	const SourceType& Source = *reinterpret_cast<const SourceType*>(Args.Source);
	QuantizedType& Target = *reinterpret_cast<QuantizedType*>(Args.Target);

	const int32 NewCount = Source.DataArray.Num();
	const int32 OldCount = static_cast<int32>(Target.Elements.Num());

	for (int32 Idx = NewCount; Idx < OldCount; ++Idx)
	{
		FreeElement(Context, Target.Elements.GetData()[Idx], DescriptorCache);
	}
	Target.Elements.AdjustSize(Context, static_cast<uint32>(NewCount));

	// ActionCollection is explicitly heterogeneous, but runs of same-type actions still benefit from memoization.
	// Keyed on UScriptStruct* (singleton per type) so same-type runs skip the path-name TMap lookup entirely.
	const UScriptStruct* LastStruct = nullptr;
	FName LastStructName;
	TRefCountPtr<const FReplicationStateDescriptor> LastDescriptor;

	FNetworkPhysicsActionElementQuantized* Elements = Target.Elements.GetData();
	for (int32 Idx = 0; Idx < NewCount; ++Idx)
	{
		FNetworkPhysicsActionElementQuantized& El = Elements[Idx];
		const TInstancedStruct<FNetworkPhysicsActionPayload>& SrcEl = Source.DataArray[Idx];

		if (!SrcEl.IsValid())
		{
			FreeElement(Context, El, DescriptorCache);
			El.StructName = FName();
			continue;
		}

		// Identify the concrete struct type and resolve descriptor / path-name only when the
		// struct pointer changes between elements.
		const UScriptStruct* Struct = SrcEl.GetScriptStruct();
		if (Struct != LastStruct)
		{
			LastStruct = Struct;
			LastStructName = DescriptorCache.GetCachedPathName(Struct);
			LastDescriptor = DescriptorCache.FindOrAddDescriptor(Struct);
		}
		const FName& PathFName = LastStructName;

		if (PathFName != El.StructName)
		{
			FreeElement(Context, El, DescriptorCache);
			checkf(LastDescriptor.IsValid(), TEXT("FNetworkPhysicsActionCollectionNetSerializer: failed to build descriptor for %s"), ToCStr(PathFName.ToString()));
			El.StructData.AdjustSize(Context, LastDescriptor->InternalSize, LastDescriptor->InternalAlignment);
			El.StructDescriptorTraits = LastDescriptor->Traits;
			El.StructName = PathFName;
		}

		FStructNetSerializerConfig StructConfig;
		StructConfig.StateDescriptor = LastDescriptor;
		FNetQuantizeArgs StructArgs = Args;
		StructArgs.Source = NetSerializerValuePointer(SrcEl.GetMemory());
		StructArgs.Target = NetSerializerValuePointer(El.StructData.GetData());
		StructArgs.NetSerializerConfig = NetSerializerConfigParam(&StructConfig);
		UE_NET_GET_SERIALIZER(FStructNetSerializer).Quantize(Context, StructArgs);
	}
}

void FNetworkPhysicsActionCollectionNetSerializer::Dequantize(FNetSerializationContext& Context, const FNetDequantizeArgs& Args)
{
	const QuantizedType& Source = *reinterpret_cast<const QuantizedType*>(Args.Source);
	SourceType& Target = *reinterpret_cast<SourceType*>(Args.Target);

	const int32 Count = static_cast<int32>(Source.Elements.Num());
	Target.DataArray.SetNum(Count);

	const FNetworkPhysicsActionElementQuantized* Elements = Source.Elements.GetData();
	for (int32 Idx = 0; Idx < Count; ++Idx)
	{
		const FNetworkPhysicsActionElementQuantized& El = Elements[Idx];
		TInstancedStruct<FNetworkPhysicsActionPayload>& DstEl = Target.DataArray[Idx];

		if (El.StructName.IsNone())
		{
			DstEl.Reset();
			continue;
		}

		const UScriptStruct* Struct = DescriptorCache.FindOrCacheStruct(El.StructName);
		if (!ensureMsgf(Struct, TEXT("FNetworkPhysicsActionCollectionNetSerializer::Dequantize: unknown struct type '%s'"), ToCStr(El.StructName.ToString())))
		{
			continue;
		}

		if (Struct != DstEl.GetScriptStruct())
		{
			DstEl.InitializeAsScriptStruct(Struct);
		}

		FStructNetSerializerConfig StructConfig;
		StructConfig.StateDescriptor = DescriptorCache.FindOrAddDescriptor(Struct);
		FNetDequantizeArgs StructArgs = Args;
		StructArgs.Source = NetSerializerValuePointer(El.StructData.GetData());
		StructArgs.Target = NetSerializerValuePointer(DstEl.GetMutableMemory());
		StructArgs.NetSerializerConfig = NetSerializerConfigParam(&StructConfig);
		UE_NET_GET_SERIALIZER(FStructNetSerializer).Dequantize(Context, StructArgs);
	}
}

void FNetworkPhysicsActionCollectionNetSerializer::Serialize(FNetSerializationContext& Context, const FNetSerializeArgs& Args)
{
	const QuantizedType& Value = *reinterpret_cast<const QuantizedType*>(Args.Source);
	const uint32 Count = Value.Elements.Num();

	FNetBitStreamWriter* Writer = Context.GetBitStreamWriter();
	WritePackedUint32(Context, Args, Count);

	if (Count == 0u)
	{
		return;
	}

	// Write collection-level "common type" name once (first valid element).
	const FNetworkPhysicsActionElementQuantized* Elements = Value.Elements.GetData();
	FName CollectionTypeName;
	for (uint32 Idx = 0u; Idx < Count; ++Idx)
	{
		if (!Elements[Idx].StructName.IsNone())
		{
			CollectionTypeName = Elements[Idx].StructName;
			break;
		}
	}

	const bool bHasValidType = !CollectionTypeName.IsNone();
	Writer->WriteBool(bHasValidType);
	if (bHasValidType)
	{
		WriteStructNameHash(*Writer, CollectionTypeName, DescriptorCache);
	}

	// Shared zero-baseline buffer, lifetime owned at this scope.
	FNetSerializerAlignedStorage SharedZeroStorage;
	ON_SCOPE_EXIT{ SharedZeroStorage.Free(Context); };

	// Intra-array delta: element[i>0] deltas against element[i-1] if same type.
	FNetSerializeDeltaArgs DeltaArgs = {};
	DeltaArgs.NetSerializerConfig = Args.NetSerializerConfig;
	for (uint32 Idx = 0u; Idx < Count; ++Idx)
	{
		const FNetworkPhysicsActionElementQuantized* Prev = (Idx > 0u && Elements[Idx].StructName == Elements[Idx - 1u].StructName)
			? &Elements[Idx - 1u] : nullptr;
		SerializeElementDelta(Context, DeltaArgs, Elements[Idx], Prev, CollectionTypeName, SharedZeroStorage, DescriptorCache);
	}
}

void FNetworkPhysicsActionCollectionNetSerializer::Deserialize(FNetSerializationContext& Context, const FNetDeserializeArgs& Args)
{
	QuantizedType& Value = *reinterpret_cast<QuantizedType*>(Args.Target);

	const uint32 NewCount = ReadPackedUint32(Context, Args);

	const uint32 OldCount = Value.Elements.Num();
	for (uint32 Idx = NewCount; Idx < OldCount; ++Idx)
	{
		FreeElement(Context, Value.Elements.GetData()[Idx], DescriptorCache);
	}
	Value.Elements.AdjustSize(Context, NewCount);

	if (NewCount == 0u)
	{
		return;
	}

	// Read collection-level "common type" name (if a valid type was sent).
	FNetBitStreamReader* Reader = Context.GetBitStreamReader();
	const bool bHasValidType = Reader->ReadBool();
	FName CollectionTypeName;
	if (bHasValidType)
	{
		CollectionTypeName = ReadStructNameHash(*Reader, DescriptorCache);
	}

	// Shared zero-baseline buffer, lifetime owned at this scope.
	FNetSerializerAlignedStorage SharedZeroStorage;
	ON_SCOPE_EXIT{ SharedZeroStorage.Free(Context); };

	// Intra-array delta: receiver passes element[i-1] as potential delta source.
	// DeserializeElementDelta determines whether to use it based on type matching (same logic as sender).
	FNetDeserializeDeltaArgs DeltaArgs = {};
	DeltaArgs.NetSerializerConfig = Args.NetSerializerConfig;
	FNetworkPhysicsActionElementQuantized* Elements = Value.Elements.GetData();
	for (uint32 Idx = 0u; Idx < NewCount; ++Idx)
	{
		const FNetworkPhysicsActionElementQuantized* Prev = (Idx > 0u) ? &Elements[Idx - 1u] : nullptr;
		DeserializeElementDelta(Context, DeltaArgs, Elements[Idx], Prev, CollectionTypeName, SharedZeroStorage, DescriptorCache);
	}
}

void FNetworkPhysicsActionCollectionNetSerializer::SerializeDelta(FNetSerializationContext& Context, const FNetSerializeDeltaArgs& Args)
{
	const QuantizedType& Curr = *reinterpret_cast<const QuantizedType*>(Args.Source);
	const QuantizedType& IrisPrev = *reinterpret_cast<const QuantizedType*>(Args.Prev);

	const uint32 CurrCount = Curr.Elements.Num();
	const uint32 PrevCount = IrisPrev.Elements.Num();

	FNetBitStreamWriter* Writer = Context.GetBitStreamWriter();
	const bool bSameCount = (CurrCount == PrevCount);
	Writer->WriteBool(bSameCount);
	if (!bSameCount)
	{
		WritePackedUint32(Context, Args, CurrCount);
	}

	if (CurrCount == 0u)
	{
		return;
	}

	const FNetworkPhysicsActionElementQuantized* CurrElements = Curr.Elements.GetData();
	const FNetworkPhysicsActionElementQuantized* PrevElements = IrisPrev.Elements.GetData();

	// Write collection-level type change flag + optional type name.
	// Find the first valid element in each array - invalid slots have StructName == None.
	FName CurrTypeName;
	for (uint32 Idx = 0u; Idx < CurrCount; ++Idx)
	{
		if (!CurrElements[Idx].StructName.IsNone())
		{
			CurrTypeName = CurrElements[Idx].StructName;
			break;
		}
	}
	FName PrevTypeName;
	for (uint32 Idx = 0u; Idx < PrevCount; ++Idx)
	{
		if (!PrevElements[Idx].StructName.IsNone())
		{
			PrevTypeName = PrevElements[Idx].StructName;
			break;
		}
	}
	const bool bCollectionTypeChanged = (CurrTypeName != PrevTypeName);
	Writer->WriteBool(bCollectionTypeChanged);
	if (bCollectionTypeChanged)
	{
		const bool bHasValidType = !CurrTypeName.IsNone();
		Writer->WriteBool(bHasValidType);
		if (bHasValidType)
		{
			WriteStructNameHash(*Writer, CurrTypeName, DescriptorCache);
		}
	}

	// Shared zero-baseline buffer, lifetime owned at this scope.
	FNetSerializerAlignedStorage SharedZeroStorage;
	ON_SCOPE_EXIT{ SharedZeroStorage.Free(Context); };

	// Chaining: same-type adjacent elements use element[i-1] as delta source.
	auto GetDeltaSource = [&](const uint32 Idx) -> const FNetworkPhysicsActionElementQuantized*
	{
		if (Idx == 0u)
		{
			return (PrevCount > 0u && !bCollectionTypeChanged) ? &PrevElements[0] : nullptr;
		}
		return (CurrElements[Idx].StructName == CurrElements[Idx - 1u].StructName) ? &CurrElements[Idx - 1u] : nullptr;
	};

	for (uint32 Idx = 0u; Idx < CurrCount; ++Idx)
	{
		SerializeElementDelta(Context, Args, CurrElements[Idx], GetDeltaSource(Idx), CurrTypeName, SharedZeroStorage, DescriptorCache);
	}
}

void FNetworkPhysicsActionCollectionNetSerializer::DeserializeDelta(FNetSerializationContext& Context, const FNetDeserializeDeltaArgs& Args)
{
	QuantizedType& Target = *reinterpret_cast<QuantizedType*>(Args.Target);
	const QuantizedType& IrisPrev = *reinterpret_cast<const QuantizedType*>(Args.Prev);

	FNetBitStreamReader* Reader = Context.GetBitStreamReader();
	uint32 NewCount = IrisPrev.Elements.Num();
	const bool bSameCount = Reader->ReadBool();
	if (!bSameCount)
	{
		NewCount = ReadPackedUint32(Context, Args);
	}

	const uint32 OldCount = Target.Elements.Num();
	for (uint32 Idx = NewCount; Idx < OldCount; ++Idx)
	{
		FreeElement(Context, Target.Elements.GetData()[Idx], DescriptorCache);
	}
	Target.Elements.AdjustSize(Context, NewCount);

	if (NewCount == 0u)
	{
		return;
	}

	const FNetworkPhysicsActionElementQuantized* PrevElements = IrisPrev.Elements.GetData();
	const uint32 PrevCount = IrisPrev.Elements.Num();

	// Read collection-level type change flag + optional type name.
	const bool bCollectionTypeChanged = Reader->ReadBool();
	FName CollectionTypeName;
	if (bCollectionTypeChanged)
	{
		const bool bHasValidType = Reader->ReadBool();
		if (bHasValidType)
		{
			CollectionTypeName = ReadStructNameHash(*Reader, DescriptorCache);
		}
	}
	else if (PrevCount > 0u)
	{
		// Type unchanged - reuse from the baseline's first valid element
		for (uint32 Idx = 0u; Idx < PrevCount; ++Idx)
		{
			if (!PrevElements[Idx].StructName.IsNone())
			{
				CollectionTypeName = PrevElements[Idx].StructName;
				break;
			}
		}
	}

	FNetworkPhysicsActionElementQuantized* Elements = Target.Elements.GetData();

	// Shared zero-baseline buffer, lifetime owned at this scope.
	FNetSerializerAlignedStorage SharedZeroStorage;
	ON_SCOPE_EXIT{ SharedZeroStorage.Free(Context); };

	// Pass the previous element as a candidate delta source - DeserializeElementDelta evaluates
	// type compatibility internally (same logic as sender) to decide whether to use Prev or a
	// zero-initialized default. We cannot check Elements[Idx].StructName here because it
	// hasn't been assigned yet (it is set inside DeserializeElementDelta via EnsureElementType).
	for (uint32 Idx = 0u; Idx < NewCount; ++Idx)
	{
		const FNetworkPhysicsActionElementQuantized* Prev = nullptr;
		if (Idx == 0u)
		{
			Prev = (PrevCount > 0u && !bCollectionTypeChanged) ? &PrevElements[0] : nullptr;
		}
		else
		{
			Prev = &Elements[Idx - 1u];
		}
		DeserializeElementDelta(Context, Args, Elements[Idx], Prev, CollectionTypeName, SharedZeroStorage, DescriptorCache);
	}
}

bool FNetworkPhysicsActionCollectionNetSerializer::IsEqual(FNetSerializationContext&, const FNetIsEqualArgs& Args)
{
	if (IsEqualImpl<QuantizedType, FNetworkPhysicsActionElementQuantized>(Args))
	{
		return true;
	}
	if (!Args.bStateIsQuantized)
	{
		const SourceType& V0 = *reinterpret_cast<const SourceType*>(Args.Source0);
		const SourceType& V1 = *reinterpret_cast<const SourceType*>(Args.Source1);
		if (V0.DataArray.Num() != V1.DataArray.Num())
		{
			return false;
		}
		for (int32 Idx = 0; Idx < V0.DataArray.Num(); ++Idx)
		{
			if (V0.DataArray[Idx] != V1.DataArray[Idx])
			{
				return false;
			}
		}
		return true;
	}
	return false;
}

bool FNetworkPhysicsActionCollectionNetSerializer::Validate(FNetSerializationContext&, const FNetValidateArgs&)
{
	return true;
}

void FNetworkPhysicsActionCollectionNetSerializer::CloneDynamicState(FNetSerializationContext& Context, const FNetCloneDynamicStateArgs& Args)
{
	CloneDynamicStateImpl<QuantizedType, FNetworkPhysicsActionElementQuantized>(Context, Args, DescriptorCache);
}

void FNetworkPhysicsActionCollectionNetSerializer::FreeDynamicState(FNetSerializationContext& Context, const FNetFreeDynamicStateArgs& Args)
{
	FreeDynamicStateImpl<QuantizedType, FNetworkPhysicsActionElementQuantized>(Context, Args, DescriptorCache);
}

void FNetworkPhysicsActionCollectionNetSerializer::CollectNetReferences(FNetSerializationContext& Context, const FNetCollectReferencesArgs& Args)
{
	CollectNetReferencesImpl<QuantizedType, FNetworkPhysicsActionElementQuantized>(Context, Args, DescriptorCache);
}

UE_NET_IMPLEMENT_FORWARDING_NETSERIALIZER_AND_REGISTRY_DELEGATES(NetworkPhysicsActionCollection, FNetworkPhysicsActionCollectionNetSerializer);

} // namespace UE::Net
