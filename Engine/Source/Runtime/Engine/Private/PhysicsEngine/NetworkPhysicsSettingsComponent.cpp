// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	NetworkPhysicsSettingsComponent.cpp
	Handles data distribution of networked physics settings to systems that need it, on both Game-Thread and Physics-Thread.
=============================================================================*/

#include "Physics/NetworkPhysicsSettingsComponent.h"
#include "Engine/Engine.h"
#include "Chaos/Declares.h"
#include "Physics/Experimental/PhysScene_Chaos.h"
#include "PBDRigidsSolver.h"
#include "PhysicsReplication.h"
#include "Components/PrimitiveComponent.h"
#include "PhysicsEngine/PhysicsObjectExternalInterface.h"
#include "PhysicsProxy/SingleParticlePhysicsProxy.h"
#include "PhysicsProxy/ClusterUnionPhysicsProxy.h"
#include "PhysicsProxy/GeometryCollectionPhysicsProxy.h"
#include "GameFramework/Pawn.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NetworkPhysicsSettingsComponent)

namespace PhysicsReplicationCVars
{
	namespace ResimulationCVars
	{
		int32 SimProxyRepMode = -1;
		static FAutoConsoleVariableRef CVarSimProxyRepMode(TEXT("np2.Resim.SimProxyRepMode"), SimProxyRepMode, TEXT("All actors with a NetworkPhysicsSettingsComponent and that are running resimulation and is ROLE_SimulatedProxy will change their physics replication mode. -1 = Disabled, 0 = Default, 1 = PredictiveInterpolation, 2 = Resimulation"));
	}
}

/** Static Initialization*/
const FNetworkPhysicsSettingsData UNetworkPhysicsSettingsComponent::SettingsNetworkPhysicsData_Default = FNetworkPhysicsSettingsData();
TMap<AActor*, UNetworkPhysicsSettingsComponent*> UNetworkPhysicsSettingsComponent::ObjectToSettings_External = TMap<AActor*, UNetworkPhysicsSettingsComponent*>();

UNetworkPhysicsSettingsComponent::UNetworkPhysicsSettingsComponent()
{
	bWantsInitializeComponent = true;
	bAutoActivate = true;
}

void UNetworkPhysicsSettingsComponent::OnRegister()
{
	Super::OnRegister();

	if (AActor* Owner = GetOwner())
	{
		if (UPrimitiveComponent* RootPrimComp = Cast<UPrimitiveComponent>(Owner->GetRootComponent()))
		{
			RootPrimComp->OnComponentPhysicsStateChanged.AddUniqueDynamic(this, &ThisClass::OnComponentPhysicsStateChanged);
		}
	}

	if (!SettingsDataAsset)
	{
		return;
	}

	// Copy over GT settings into a TSharedPtr to be accessed on the physics thread
	SettingsDataAsset->InitializeInternalSettings();

	using namespace Chaos;
	NetworkPhysicsSettings_Internal = nullptr;
	if (UWorld* World = GetWorld())
	{
		if (FPhysScene* PhysScene = World->GetPhysicsScene())
		{
			if (Chaos::FPhysicsSolver* Solver = PhysScene->GetSolver())
			{
				NetworkPhysicsSettings_Internal = Solver->CreateAndRegisterSimCallbackObject_External<FNetworkPhysicsSettingsComponentAsync>();
				
				// Marshal settings data from GT to PT
				if (NetworkPhysicsSettings_Internal)
				{
					if (AActor* Owner = GetOwner())
					{
						if (UPrimitiveComponent* RootPrimComp = Cast<UPrimitiveComponent>(Owner->GetRootComponent()))
						{
							if (Chaos::FPhysicsObjectHandle PhysicsObject = RootPrimComp->GetPhysicsObjectByName(NAME_None))
							{
								if (FNetworkPhysicsSettingsAsyncInput* AsyncInput = NetworkPhysicsSettings_Internal->GetProducerInputData_External())
								{
									AsyncInput->PhysicsObject = PhysicsObject;
									AsyncInput->Settings_Internal = GetSettings_Internal();

#if WITH_EDITORONLY_DATA
									PRAGMA_DISABLE_DEPRECATION_WARNINGS
									FNetworkPhysicsSettingsAsync SettingsCopy
									{
										.GeneralSettings = GeneralSettings,
										.DefaultReplicationSettings = DefaultReplicationSettings,
										.PredictiveInterpolationSettings = PredictiveInterpolationSettings,
										.ResimulationSettings = ResimulationSettings,
										.NetworkPhysicsComponentSettings = NetworkPhysicsComponentSettings,
									};
									AsyncInput->Settings = SettingsCopy;
									PRAGMA_ENABLE_DEPRECATION_WARNINGS
#endif
								}

								// Apply resimulation error correction settings for render interpolation to the physics proxy
								GetSettings().ResimulationSettings.ResimulationErrorCorrectionSettings.ApplySettings_External(PhysicsObject);
							}
						}
					}
				}
			}
		}
	}

	if (AActor* Owner = GetOwner())
	{
		UNetworkPhysicsSettingsComponent::ObjectToSettings_External.Add(Owner, this);
	}
}

void UNetworkPhysicsSettingsComponent::OnUnregister()
{
	Super::OnUnregister();

	if (AActor* Owner = GetOwner())
	{
		if (UPrimitiveComponent* RootPrimComp = Cast<UPrimitiveComponent>(Owner->GetRootComponent()))
		{
			RootPrimComp->OnComponentPhysicsStateChanged.RemoveDynamic(this, &ThisClass::OnComponentPhysicsStateChanged);
		}
	}

	// Unbind from pawn controller change delegate and unregister focal particle
	if (APawn* OwnerPawn = Cast<APawn>(GetOwner()))
	{
		OwnerPawn->ReceiveControllerChangedDelegate.RemoveDynamic(this, &ThisClass::OnControllerChanged);
	}
	UnregisterFromPhysicsReplicationLOD();

	if (SettingsDataAsset)
	{
		SettingsDataAsset->MarkUninitialized();
	}

	using namespace Chaos;
	if (NetworkPhysicsSettings_Internal)
	{
		if (UWorld* World = GetWorld())
		{
			if (FPhysScene* PhysScene = World->GetPhysicsScene())
			{
				if (Chaos::FPhysicsSolver* Solver = PhysScene->GetSolver())
				{
					Solver->UnregisterAndFreeSimCallbackObject_External(NetworkPhysicsSettings_Internal);
				}
			}
		}
	}
	NetworkPhysicsSettings_Internal = nullptr;

	if (AActor* Owner = GetOwner())
	{
		UNetworkPhysicsSettingsComponent::ObjectToSettings_External.Remove(Owner);
	}
}

void UNetworkPhysicsSettingsComponent::BeginPlay()
{
	Super::BeginPlay();

	// Apply overrides on actor
	if (AActor* Owner = GetOwner())
	{
		if ((GetSettings().GeneralSettings.bOverrideSimProxyRepMode || PhysicsReplicationCVars::ResimulationCVars::SimProxyRepMode >= 0)
			&& Owner->GetLocalRole() == ENetRole::ROLE_SimulatedProxy)
		{
			EPhysicsReplicationMode RepMode = GetSettings().GeneralSettings.bOverrideSimProxyRepMode ? GetSettings().GeneralSettings.SimProxyRepMode : static_cast<EPhysicsReplicationMode>(PhysicsReplicationCVars::ResimulationCVars::SimProxyRepMode);
			Owner->SetPhysicsReplicationMode(RepMode);
		}
	}
	RegisterInPhysicsReplicationLOD();

	// Bind to pawn controller change delegate to register/unregister focal particle on possession changes
	if (APawn* OwnerPawn = Cast<APawn>(GetOwner()))
	{
		OwnerPawn->ReceiveControllerChangedDelegate.AddUniqueDynamic(this, &ThisClass::OnControllerChanged);
	}
}

void UNetworkPhysicsSettingsComponent::OnComponentPhysicsStateChanged(UPrimitiveComponent* ChangedComponent, EComponentPhysicsStateChange StateChange)
{
	if (StateChange == EComponentPhysicsStateChange::Created)
	{
		if (Chaos::FPhysicsObjectHandle PhysicsObject = ChangedComponent->GetPhysicsObjectByName(NAME_None))
		{
			if (NetworkPhysicsSettings_Internal)
			{
				if (FNetworkPhysicsSettingsAsyncInput* AsyncInput = NetworkPhysicsSettings_Internal->GetProducerInputData_External())
				{
					AsyncInput->PhysicsObject = PhysicsObject;
				}
			}

			// Apply resimulation error correction settings for render interpolation to the physics proxy
			GetSettings().ResimulationSettings.ResimulationErrorCorrectionSettings.ApplySettings_External(PhysicsObject);
		}

		RegisterInPhysicsReplicationLOD();
	}
}

void UNetworkPhysicsSettingsComponent::RegisterInPhysicsReplicationLOD()
{
	if (!GetSettings().GeneralSettings.bFocalParticleInPhysicsReplicationLOD)
	{
		return;
	}

	if (AActor* Owner = GetOwner())
	{
		if (Owner->GetLocalRole() == ENetRole::ROLE_AutonomousProxy)
		{
			Owner->RegisterAsFocalPointInPhysicsReplicationLOD();
		}
	}
}

void UNetworkPhysicsSettingsComponent::UnregisterFromPhysicsReplicationLOD()
{
	if (AActor* Owner = GetOwner())
	{
		Owner->UnregisterAsFocalPointInPhysicsReplicationLOD();
	}
}

void UNetworkPhysicsSettingsComponent::OnControllerChanged(APawn* InPawn, AController* OldController, AController* NewController)
{
	if (!GetSettings().GeneralSettings.bFocalParticleInPhysicsReplicationLOD)
	{
		return;
	}

	AActor* Owner = GetOwner();
	if (!Owner)
	{
		return;
	}

	if (NewController != nullptr && Owner->GetLocalRole() == ENetRole::ROLE_AutonomousProxy)
	{
		Owner->RegisterAsFocalPointInPhysicsReplicationLOD();
	}
	else
	{
		UnregisterFromPhysicsReplicationLOD();
	}
}

UNetworkPhysicsSettingsComponent* UNetworkPhysicsSettingsComponent::GetSettingsForActor(AActor* Owner)
{
	UNetworkPhysicsSettingsComponent** Value = UNetworkPhysicsSettingsComponent::ObjectToSettings_External.Find(Owner);
	return Value ? *Value : nullptr;
}

void FNetworkPhysicsSettingsResimulationErrorCorrection::ApplySettings_External(Chaos::FPhysicsObjectHandle PhysicsObject, const bool bInForceOverride/* = false*/) const
{
	if (!PhysicsObject)
	{
		return;
	}

	if (bOverrideResimErrorInterpolationSettings || bInForceOverride)
	{
		FLockedWritePhysicsObjectExternalInterface Interface = FPhysicsObjectExternalInterface::LockWrite({ &PhysicsObject, 1 });

		if (const Chaos::FGeometryParticle* Particle = Interface->GetParticle(PhysicsObject))
		{
			if (!Particle->GetProxy())
			{
				return;
			}

			auto ApplySettingsHelper = [this](auto Proxy)
			{
				// Get the proxies error interpolation, create one if not already present
				if (FProxyInterpolationError* InterpolationError = Proxy->template GetOrCreateErrorInterpolationData<FProxyInterpolationError>())
				{
					// Apply the custom settings
					FErrorInterpolationSettings& Settings = InterpolationError->GetOrCreateErrorInterpolationSettings();
					Settings.ErrorCorrectionDuration = ResimErrorCorrectionDuration;
					Settings.MaximumErrorCorrectionBeforeSnapping = ResimErrorMaximumDistanceBeforeSnapping;
					Settings.MaximumErrorCorrectionDesyncTimeBeforeSnapping = ResimErrorMaximumDesyncTimeBeforeSnapping;
					Settings.ErrorDirectionalDecayMultiplier = ResimErrorDirectionalDecayMultiplier;
					Settings.bApplyExponentialDecay = bRenderInterpApplyExponentialDecay;
					Settings.ExponentialDecayLinearHalfLife = RenderInterpExponentialDecayLinearHalfLife;
					Settings.ExponentialDecayAngularHalfLife = RenderInterpExponentialDecayAngularHalfLife;
					Settings.MinimumLinearThreshold = RenderInterpMinimumLinearThreshold;
					Settings.MinimumAngularThreshold = RenderInterpMinimumAngularThreshold;
				}
			};

			switch (Particle->GetProxy()->GetType())
			{
				case EPhysicsProxyType::SingleParticleProxy:
				{
					Chaos::FSingleParticlePhysicsProxy* SPProxy = static_cast<Chaos::FSingleParticlePhysicsProxy*>(Particle->GetProxy());
					if (ensure(SPProxy))
					{
						ApplySettingsHelper(SPProxy);
					}
					break;
				}
				case EPhysicsProxyType::ClusterUnionProxy:
				{
					Chaos::FClusterUnionPhysicsProxy* CUProxy = static_cast<Chaos::FClusterUnionPhysicsProxy*>(Particle->GetProxy());
					if (ensure(CUProxy))
					{
						ApplySettingsHelper(CUProxy);
					}
					break;
				}
				case EPhysicsProxyType::GeometryCollectionType:
				{
					FGeometryCollectionPhysicsProxy* GCProxy = static_cast<FGeometryCollectionPhysicsProxy*>(Particle->GetProxy());
					if (ensure(GCProxy))
					{
						ApplySettingsHelper(GCProxy);
					}
					break;
				}
				default:
				{
					ensure(false);
					break;
				}
			};
		}
	}
}

void FNetworkPhysicsSettingsResimulationErrorCorrection::ResetSettings_External(Chaos::FPhysicsObjectHandle PhysicsObject)
{
	if (!PhysicsObject)
	{
		return;
	}

	FLockedWritePhysicsObjectExternalInterface Interface = FPhysicsObjectExternalInterface::LockWrite({ &PhysicsObject, 1 });

	if (const Chaos::FGeometryParticle* Particle = Interface->GetParticle(PhysicsObject))
	{
		if (!Particle->GetProxy())
		{
			return;
		}

		auto ResetSettingsHelper = [](auto Proxy)
			{
				if (FProxyInterpolationError* InterpolationError = Proxy->template GetOrCreateErrorInterpolationData<FProxyInterpolationError>())
				{
					InterpolationError->ResetErrorInterpolationSettings();
				}
			};

		switch (Particle->GetProxy()->GetType())
		{
			case EPhysicsProxyType::SingleParticleProxy:
			{
				Chaos::FSingleParticlePhysicsProxy* SPProxy = static_cast<Chaos::FSingleParticlePhysicsProxy*>(Particle->GetProxy());
				if (ensure(SPProxy))
				{
					ResetSettingsHelper(SPProxy);
				}
				break;
			}
			case EPhysicsProxyType::ClusterUnionProxy:
			{
				Chaos::FClusterUnionPhysicsProxy* CUProxy = static_cast<Chaos::FClusterUnionPhysicsProxy*>(Particle->GetProxy());
				if (ensure(CUProxy))
				{
					ResetSettingsHelper(CUProxy);
				}
				break;
			}
			case EPhysicsProxyType::GeometryCollectionType:
			{
				FGeometryCollectionPhysicsProxy* GCProxy = static_cast<FGeometryCollectionPhysicsProxy*>(Particle->GetProxy());
				if (ensure(GCProxy))
				{
					ResetSettingsHelper(GCProxy);
				}
				break;
			}
			default:
			{
				ensure(false);
				break;
			}
		};
	}
}

bool FNetworkPhysicsSettingsResimulationErrorCorrection::FromPhysicObject(Chaos::FPhysicsObjectHandle PhysicsObject, FNetworkPhysicsSettingsResimulationErrorCorrection& OutValue)
{
	if (!PhysicsObject)
	{
		return false;
	}

	FLockedReadPhysicsObjectExternalInterface Interface = FPhysicsObjectExternalInterface::LockRead({ &PhysicsObject, 1 });

	if (const Chaos::FGeometryParticle* Particle = Interface->GetParticle(PhysicsObject))
	{
		if (!Particle->GetProxy())
		{
			return false;
		}

		auto GetSettingsHelper = [&OutValue](auto Proxy)
		{
			// Get the proxies error interpolation, create one if not already present
			if (const FProxyInterpolationError* InterpolationError = Proxy->template GetErrorInterpolationData<FProxyInterpolationError>())
			{
				// Apply the custom settings
				if (const FErrorInterpolationSettings* Settings = InterpolationError->GetErrorInterpolationSettings())
				{
					OutValue.ResimErrorCorrectionDuration = Settings->ErrorCorrectionDuration;
					OutValue.ResimErrorMaximumDistanceBeforeSnapping = Settings->MaximumErrorCorrectionBeforeSnapping;
					OutValue.ResimErrorMaximumDesyncTimeBeforeSnapping = Settings->MaximumErrorCorrectionDesyncTimeBeforeSnapping;
					OutValue.ResimErrorDirectionalDecayMultiplier = Settings->ErrorDirectionalDecayMultiplier;
					OutValue.bRenderInterpApplyExponentialDecay = Settings->bApplyExponentialDecay;
					OutValue.RenderInterpExponentialDecayLinearHalfLife = Settings->ExponentialDecayLinearHalfLife;
					OutValue.RenderInterpExponentialDecayAngularHalfLife = Settings->ExponentialDecayAngularHalfLife;
					OutValue.RenderInterpMinimumLinearThreshold = Settings->MinimumLinearThreshold;
					OutValue.RenderInterpMinimumAngularThreshold = Settings->MinimumAngularThreshold;
					return true;
				}
			}
			return false;
		};

		switch (Particle->GetProxy()->GetType())
		{
			case EPhysicsProxyType::SingleParticleProxy:
			{
				const Chaos::FSingleParticlePhysicsProxy* SPProxy = static_cast<Chaos::FSingleParticlePhysicsProxy*>(Particle->GetProxy());
				if (ensure(SPProxy))
				{
					return GetSettingsHelper(SPProxy);
				}
				break;
			}
			case EPhysicsProxyType::ClusterUnionProxy:
			{
				const Chaos::FClusterUnionPhysicsProxy* CUProxy = static_cast<Chaos::FClusterUnionPhysicsProxy*>(Particle->GetProxy());
				if (ensure(CUProxy))
				{
					return GetSettingsHelper(CUProxy);
				}
				break;
			}
			case EPhysicsProxyType::GeometryCollectionType:
			{
				const FGeometryCollectionPhysicsProxy* GCProxy = static_cast<FGeometryCollectionPhysicsProxy*>(Particle->GetProxy());
				if (ensure(GCProxy))
				{
					return GetSettingsHelper(GCProxy);
				}
				break;
			}
			default:
			{
				ensure(false);
				return false;
			}
		};
	}
	return false;
}

#pragma region // FNetworkPhysicsSettingsComponentAsync

FNetworkPhysicsSettingsComponentAsync::FNetworkPhysicsSettingsComponentAsync() : TSimCallbackObject()
	, Settings()
	, PhysicsObject(nullptr)
{}

void FNetworkPhysicsSettingsComponentAsync::OnPreSimulate_Internal()
{
	ConsumeAsyncInput();
};

void FNetworkPhysicsSettingsComponentAsync::ConsumeAsyncInput()
{
	// Receive data on PT from GT
	if (const FNetworkPhysicsSettingsAsyncInput* AsyncInput = GetConsumerInput_Internal())
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		if (AsyncInput->Settings.IsSet())
		{
			Settings = *AsyncInput->Settings;
		}
		PRAGMA_ENABLE_DEPRECATION_WARNINGS

		if (AsyncInput->Settings_Internal.IsSet() && (*AsyncInput->Settings_Internal).IsValid())
		{
			Settings_Internal = (*AsyncInput->Settings_Internal).Pin();
		}

		if (AsyncInput->PhysicsObject.IsSet())
		{
			PhysicsObject = *AsyncInput->PhysicsObject;
			RegisterSettingsInPhysicsReplication();
		}
	}
}

void FNetworkPhysicsSettingsComponentAsync::RegisterSettingsInPhysicsReplication()
{
	if (Chaos::FPBDRigidsSolver* RigidsSolver = static_cast<Chaos::FPBDRigidsSolver*>(GetSolver()))
	{
		if (IPhysicsReplicationAsync* PhysRep = RigidsSolver->GetPhysicsReplication_Internal())
		{
			PhysRep->RegisterSettings(PhysicsObject, Settings_Internal.ToWeakPtr());
		}
	}
}

#pragma endregion // FNetworkPhysicsSettingsComponentAsync
