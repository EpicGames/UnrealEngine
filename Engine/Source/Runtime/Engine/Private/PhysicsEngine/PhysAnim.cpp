// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	PhysAnim.cpp: Code for supporting animation/physics blending
=============================================================================*/ 

#include "Components/SkeletalMeshComponent.h"
#include "Animation/AnimStats.h"
#include "Engine/World.h"
#include "SkeletalRenderPublic.h"
#include "Components/LineBatchComponent.h"
#include "Math/QuatRotationTranslationMatrix.h"
#include "Rendering/SkeletalMeshRenderData.h"
#include "Physics/Experimental/PhysScene_Chaos.h"
#include "PhysicsEngine/PhysicsConstraintTemplate.h"
#include "PhysicsEngine/PhysicsAsset.h"
#include "PhysicsEngine/SkeletalBodySetup.h"

CSV_DECLARE_CATEGORY_MODULE_EXTERN(CORE_API, Basic);


/** Used for drawing pre-phys skeleton if bShowPrePhysBones is true */
static const FColor AnimSkelDrawColor(255, 64, 64);

// Temporary workspace for caching world-space matrices.
struct FAssetWorldBoneTM
{
	FTransform	TM;			// Should never contain scaling.
	bool bUpToDate;			// If this equals PhysAssetUpdateNum, then the matrix is up to date.
};


FAutoConsoleTaskPriority CPrio_FParallelBlendPhysicsTask(
	TEXT("TaskGraph.TaskPriorities.ParallelBlendPhysicsTask"),
	TEXT("Task and thread priority for FParallelBlendPhysicsTask."),
	ENamedThreads::HighThreadPriority, // if we have high priority task threads, then use them...
	ENamedThreads::NormalTaskPriority, // .. at normal task priority
	ENamedThreads::HighTaskPriority // if we don't have hi pri threads, then use normal priority threads at high task priority instead
	);

class FParallelBlendPhysicsTask
{
	TWeakObjectPtr<USkeletalMeshComponent> SkeletalMeshComponent;

public:
	FParallelBlendPhysicsTask(TWeakObjectPtr<USkeletalMeshComponent> InSkeletalMeshComponent)
		: SkeletalMeshComponent(InSkeletalMeshComponent)
	{
	}

	static FORCEINLINE TStatId GetStatId()
	{
		RETURN_QUICK_DECLARE_CYCLE_STAT(FParallelBlendPhysicsTask, STATGROUP_TaskGraphTasks);
	}
	static FORCEINLINE ENamedThreads::Type GetDesiredThread()
	{
		return CPrio_FParallelBlendPhysicsTask.Get();
	}
	static FORCEINLINE ESubsequentsMode::Type GetSubsequentsMode()
	{
		return ESubsequentsMode::TrackSubsequents;
	}

	void DoTask(ENamedThreads::Type CurrentThread, const FGraphEventRef& MyCompletionGraphEvent)
	{
		if (USkeletalMeshComponent* Comp = SkeletalMeshComponent.Get())
		{
			SCOPED_NAMED_EVENT(FParallelBlendPhysicsTask_DoTask, FColor::Yellow);
			Comp->ParallelBlendPhysics();
		}
	}
};

class FParallelBlendPhysicsCompletionTask
{
	TWeakObjectPtr<USkeletalMeshComponent> SkeletalMeshComponent;

public:
	FParallelBlendPhysicsCompletionTask(TWeakObjectPtr<USkeletalMeshComponent> InSkeletalMeshComponent)
		: SkeletalMeshComponent(InSkeletalMeshComponent)
	{
	}

	FORCEINLINE TStatId GetStatId() const
	{
		RETURN_QUICK_DECLARE_CYCLE_STAT(FParallelBlendPhysicsCompletionTask, STATGROUP_TaskGraphTasks);
	}
	static ENamedThreads::Type GetDesiredThread()
	{
		return ENamedThreads::GameThread;
	}
	static ESubsequentsMode::Type GetSubsequentsMode()
	{
		return ESubsequentsMode::TrackSubsequents;
	}

	void DoTask(ENamedThreads::Type CurrentThread, const FGraphEventRef& MyCompletionGraphEvent)
	{
		CSV_SCOPED_TIMING_STAT_EXCLUSIVE(Physics);
		SCOPED_NAMED_EVENT(FParallelBlendPhysicsCompletionTask_DoTask, FColor::Yellow);
		SCOPE_CYCLE_COUNTER(STAT_AnimGameThreadTime);

		if (USkeletalMeshComponent* Comp = SkeletalMeshComponent.Get())
		{
			Comp->CompleteParallelBlendPhysics();
		}
	}
};

typedef TArray<FAssetWorldBoneTM, TMemStackAllocator<alignof(FAssetWorldBoneTM)>> TAssetWorldBoneTMArray;
// Use current pose to calculate world-space position of this bone without physics now.
void UpdateWorldBoneTM(TAssetWorldBoneTMArray& WorldBoneTMs, const TArray<FTransform>& InBoneSpaceTransforms, int32 BoneIndex, USkeletalMeshComponent* SkelComp, const FTransform &LocalToWorldTM, const FVector& Scale3D)
{
	// If its already up to date - do nothing
	if(	WorldBoneTMs[BoneIndex].bUpToDate )
	{
		return;
	}

	FTransform ParentTM, RelTM;
	if(BoneIndex == 0)
	{
		// If this is the root bone, we use the mesh component LocalToWorld as the parent transform.
		ParentTM = LocalToWorldTM;
	}
	else
	{
		// If not root, use our cached world-space bone transforms.
		int32 ParentIndex = SkelComp->GetSkeletalMeshAsset()->GetRefSkeleton().GetParentIndex(BoneIndex);
		UpdateWorldBoneTM(WorldBoneTMs, InBoneSpaceTransforms, ParentIndex, SkelComp, LocalToWorldTM, Scale3D);
		ParentTM = WorldBoneTMs[ParentIndex].TM;
	}

	if (InBoneSpaceTransforms.IsValidIndex(BoneIndex))
	{
		RelTM = InBoneSpaceTransforms[BoneIndex];
		RelTM.ScaleTranslation(Scale3D);

		WorldBoneTMs[BoneIndex].TM = RelTM * ParentTM;
		WorldBoneTMs[BoneIndex].bUpToDate = true;
	}
}
TAutoConsoleVariable<int32> CVarPhysicsAnimBlendUpdatesPhysX(TEXT("p.PhysicsAnimBlendUpdatesPhysX"), 1, TEXT("Whether to update the physx simulation with the results of physics animation blending"));

void USkeletalMeshComponent::PerformBlendPhysicsBones(
	const TArray<FBoneIndexType>& InRequiredBones, 
	TArray<FTransform>& InOutComponentSpaceTransforms, 
	TArray<FTransform>& InOutBoneSpaceTransforms)
{
	SCOPE_CYCLE_COUNTER(STAT_BlendInPhysics);

	UPhysicsAsset * const PhysicsAsset = GetPhysicsAsset();
	check( PhysicsAsset );

	if (InOutComponentSpaceTransforms.Num() == 0)
	{
		return;
	}

	if (InOutBoneSpaceTransforms.Num() == 0)
	{
		return;
	}

	// Get the scene, and do nothing if we can't get one.
	FPhysScene* PhysScene = nullptr;
	if (GetWorld() != nullptr)
	{
		PhysScene = GetWorld()->GetPhysicsScene();
	}

	if (PhysScene == nullptr)
	{
		return;
	}

	FMemMark Mark(FMemStack::Get());
	// Make sure scratch space is big enough.
	TAssetWorldBoneTMArray WorldBoneTMs;
	WorldBoneTMs.AddZeroed(InOutComponentSpaceTransforms.Num());
	
	FTransform LocalToWorldTM = GetComponentTransform();
	LocalToWorldTM.RemoveScaling();

	// This will update both InOutComponentSpaceTransforms and InOutBoneSpaceTransforms for every
	// required bone, leaving any others unchanged.
	FPhysicsCommand::ExecuteRead(this, [&]()
	{
		// Note that IsInstanceSimulatingPhysics returns false for kinematic bodies, so
		// bSimulatedRootBody means "is the root physics body dynamic" (not the same as the root bone)
		const bool bSimulatedRootBody = Bodies.IsValidIndex(RootBodyData.BodyIndex) && 
			Bodies[RootBodyData.BodyIndex]->IsInstanceSimulatingPhysics();

		// Get the anticipated component transform - noting that if PhysicsTransformUpdateMode is
		// set to SimulationUpatesComponentTransform then this will come from the simulation.
		const FTransform NewComponentTransform = GetComponentTransformFromBodyInstance(Bodies[RootBodyData.BodyIndex]);

		// Get component scale. If we have a body on the root bone, this will be the scale applied 
		// to its relative physics position.
		FVector TotalScale3D = GetComponentTransform().GetScale3D();
		FVector RecipScale3D = TotalScale3D.Reciprocal();

		// For bodies that are not at the root bone, we will also need to apply the inverse of
		// the root bone scale to the body positions reported by physics. We will update the
		// scale below when we have processed the body on the root bone (if there is one)
		const FVector RootBoneScale = InOutBoneSpaceTransforms[0].GetScale3D();
		const bool bUnitScaledRootBone = RootBoneScale.Equals(FVector(1, 1, 1), UE_SMALL_NUMBER);
		bool bHaveSetParentScale = bUnitScaledRootBone;	// We don't need to change the scale if root has no scale

		// For each bone:
		// * Update the WorldBoneTMs entry
		// * Update InOutBoneSpaceTransforms, using the physics blend weights
		// * Update InOutComponentSpaceTransforms using transforms calculated from the bone space transforms
		//
		// This prevents intermediate physics blend weights from "breaking" joints (e.g. if the
		// weight was just applied in world space).
		for(int32 i=0; i<InRequiredBones.Num(); i++)
		{
			// Gets set to true if we have a valid body and the skeletal mesh option has been set to
			// be driven by kinematic body parts.
			bool bDriveMeshWhenKinematic = false;

			int32 BoneIndex = InRequiredBones[i];

			// See if this is a physics bone - i.e. if there is a body registered to/associated with it.
			// If so - get its world space matrix and its parents world space matrix and calc relative atom.
			int32 BodyIndex = PhysicsAsset->FindBodyIndex(GetSkeletalMeshAsset()->GetRefSkeleton().GetBoneName(BoneIndex));
			if(BodyIndex != INDEX_NONE )
			{	
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
				// tracking down TTP 280421. Remove this if this doesn't happen. 
				if ( !ensure(Bodies.IsValidIndex(BodyIndex)) )
				{
					UE_LOGF(LogPhysics, Warning, "%ls(Mesh %ls, PhysicsAsset %ls)", 
						*GetName(), *GetNameSafe(GetSkeletalMeshAsset()), *GetNameSafe(PhysicsAsset));
					UE_LOGF(LogPhysics, Warning, " - # of BodySetup (%d), # of Bodies (%d), Invalid BodyIndex(%d)", 
						PhysicsAsset->SkeletalBodySetups.Num(), Bodies.Num(), BodyIndex);
					continue;
				}
#endif
				FBodyInstance* PhysicsAssetBodyInstance = Bodies[BodyIndex];

				// If this body is welded to something, it will not have an updated transform so we will use the
				// bone transform. This means that if the mesh continues to play an animation, the pose will not 
				// match the pose when the weld happened. Ideally we would restore the relative transform at the
				// time of the weld, but we do not explicitly store that data (though it could perhaps be 
				// recovered from the collision geometry hierarchy, it's easy to just not animate.)
				if (PhysicsAssetBodyInstance->WeldParent != nullptr)
				{
					BodyIndex = INDEX_NONE;
				}

				bDriveMeshWhenKinematic =
					bUpdateMeshWhenKinematic &&
					PhysicsAssetBodyInstance->IsValidBodyInstance();

				// Only process this bone if it is simulated, or required due to the bDriveMeshWhenKinematic flag
				// Also, if the root bone is scaled, we need to update WorldBoneTMs, even for Kinematics that are 
				// not blended into the output because they may be a parent body of a dynamic body
				const bool bIsSimulatingPhysics = PhysicsAssetBodyInstance->IsInstanceSimulatingPhysics();
				if (bIsSimulatingPhysics || bDriveMeshWhenKinematic || !bUnitScaledRootBone)
				{
					FTransform PhysTM = PhysicsAssetBodyInstance->GetUnrealWorldTransform_AssumesLocked();

					// Store this world-space transform in cache.
					WorldBoneTMs[BoneIndex].TM = PhysTM;
					WorldBoneTMs[BoneIndex].bUpToDate = true;

					if (PhysicsAssetBodyInstance->IsPhysicsDisabled())
					{
						continue;
					}

					// NOTE: if the root bone is scaled we need to update the WorldBoneTMs above for both
					// kinematic and dynamic bodies so that the parent pose calculation below is correct. 
					// For kinematics that are not blended into the output pose, we are done here
					if (!bIsSimulatingPhysics && !bDriveMeshWhenKinematic)
					{
						continue;
					}

					// Bodies that are not on the root bone must have the position reported by physics
					// inverse scaled by the root bone scale.
					if ((BoneIndex > 0) && !bHaveSetParentScale)
					{
						TotalScale3D *= RootBoneScale;
						RecipScale3D = TotalScale3D.Reciprocal();
						bHaveSetParentScale = true;
					}

					// Note that bBlendPhysics is a flag that is used to force use of the physics
					// body, irrespective of whether they are simulated or not, or the value of the
					// physics blend weight.
					//
					// Note that the existence of kinematic body parts towards/at the root can be a
					// problem when they have a blend weight of zero in the asset, which is the
					// default. This will result in InOutBoneSpaceTransforms not including the
					// transform offset that goes to the real-world physics locations of body parts,
					// so rag-dolls (etc) will get translated with the movement component. If this
					// happens and is not the desired behavior, the solution to this is for the
					// owner to explicitly set the physics blend weight of kinematic parts to one.
					float PhysicsBlendWeight = bBlendPhysics ? 1.f : PhysicsAssetBodyInstance->PhysicsBlendWeight;

					// If the body instance is disabled, then we want to use the animation transform
					// and ignore the physics one
					if (PhysicsAssetBodyInstance->IsPhysicsDisabled())
					{
						PhysicsBlendWeight = 0.0f;
					}

					// Only do the calculations here if there is a PhysicsBlendWeight, since in the
					// end we will use this weight to blend from the input/animation value to the
					// physical one.
					if (PhysicsBlendWeight > 0.f)
					{
						if (!(ensure(InOutBoneSpaceTransforms.Num())))
						{
							continue;
						}

						// Find the transform of the parent of this bone.
						FTransform ParentWorldTM;
						if (BoneIndex == 0)
						{
							ParentWorldTM = LocalToWorldTM;
						}
						else
						{
							// If not root, get parent TM from cache (making sure its up-to-date).
							int32 ParentIndex = GetSkeletalMeshAsset()->GetRefSkeleton().GetParentIndex(BoneIndex);
							UpdateWorldBoneTM(WorldBoneTMs, InOutBoneSpaceTransforms, ParentIndex, this, LocalToWorldTM, TotalScale3D);
							ParentWorldTM = WorldBoneTMs[ParentIndex].TM;
						}

						// Calculate the relative transform of the current and parent (physical) bones.
						FTransform RelTM = PhysTM.GetRelativeTransform(ParentWorldTM);
						RelTM.RemoveScaling();	// NOTE: this also normalizes the rotation
						FQuat RelRot(RelTM.GetRotation());
						FVector RelPos =  RecipScale3D * RelTM.GetLocation();
						FTransform PhysicalBoneSpaceTransform = FTransform(
							RelRot, RelPos, InOutBoneSpaceTransforms[BoneIndex].GetScale3D());

						// Now blend in this atom. See if we are forcing this bone to always be blended in
						InOutBoneSpaceTransforms[BoneIndex].Blend(
							InOutBoneSpaceTransforms[BoneIndex], PhysicalBoneSpaceTransform, PhysicsBlendWeight);
					}
				}
			}

			if (!(ensure(BoneIndex < InOutComponentSpaceTransforms.Num())))
			{
				continue;
			}

			// Update InOutComponentSpaceTransforms entry for this bone now - it will be the parent
			// component-space transform, offset with the current bone-space transform.
			if( BoneIndex == 0 )
			{
				if (!(ensure(InOutBoneSpaceTransforms.Num())))
				{
					continue;
				}
				InOutComponentSpaceTransforms[0] = InOutBoneSpaceTransforms[0];
			}
			else
			{
				if(bDriveMeshWhenKinematic || bLocalSpaceKinematics || BodyIndex == INDEX_NONE || Bodies[BodyIndex]->IsInstanceSimulatingPhysics())
				{
					if (!(ensure(BoneIndex < InOutBoneSpaceTransforms.Num())))
					{
						continue;
					}
					const int32 ParentIndex = GetSkeletalMeshAsset()->GetRefSkeleton().GetParentIndex(BoneIndex);
					InOutComponentSpaceTransforms[BoneIndex] = InOutBoneSpaceTransforms[BoneIndex] * InOutComponentSpaceTransforms[ParentIndex];

					/**
					* Normalize rotations.
					* We want to remove any loss of precision due to accumulation of error.
					* i.e. A componentSpace transform is the accumulation of all of its local space parents. The further down the chain, the greater the error.
					* SpaceBases are used by external systems, we feed this to PhysX, send this to gameplay through bone and socket queries, etc.
					* So this is a good place to make sure all transforms are normalized.
					*/
					InOutComponentSpaceTransforms[BoneIndex].NormalizeRotation();
				}
				else if(bSimulatedRootBody)
				{
					InOutComponentSpaceTransforms[BoneIndex] = 
						Bodies[BodyIndex]->GetUnrealWorldTransform_AssumesLocked().GetRelativeTransform(NewComponentTransform);
				}
			}
		}
	});	//end scope for read lock

}



bool USkeletalMeshComponent::ShouldBlendPhysicsBones() const
{
	return	(Bodies.Num() > 0) &&
			(CollisionEnabledHasPhysics(GetCollisionEnabled())) &&
			(bBlendPhysics || DoAnyPhysicsBodiesHaveWeight());
}

bool USkeletalMeshComponent::DoAnyPhysicsBodiesHaveWeight() const
{
	for (const FBodyInstance* Body : Bodies)
	{
		if (Body && Body->PhysicsBlendWeight > 0.f)
		{
			return true;
		}
	}

	return false;
}

TAutoConsoleVariable<int32> CVarUseParallelBlendPhysics(TEXT("a.ParallelBlendPhysics"), 1, TEXT("If 1, physics blending will be run across the task graph system. If 0, blending will run purely on the game thread"));

void USkeletalMeshComponent::BlendInPhysicsInternal(FTickFunction& ThisTickFunction)
{
	check(IsInGameThread());

	// Can't do anything without a SkeletalMesh
	if( !GetSkeletalMeshAsset())
	{
		return;
	}

	// We now have all the animations blended together and final relative transforms for each bone.
	// If we don't have or want any physics, we do nothing.
	if( Bodies.Num() > 0 && CollisionEnabledHasPhysics(GetCollisionEnabled()) )
	{
		HandleExistingParallelEvaluationTask(/*bBlockOnTask = */ true, /*bPerformPostAnimEvaluation =*/ true);
		// start parallel work
		check(!IsValidRef(ParallelAnimationEvaluationTask));

		const bool bParallelBlend = !!CVarUseParallelBlendPhysics.GetValueOnGameThread() && FApp::ShouldUseThreadingForPerformance();
		if(bParallelBlend)
		{
			SwapEvaluationContextBuffers();

			ParallelAnimationEvaluationTask = TGraphTask<FParallelBlendPhysicsTask>::CreateTask().ConstructAndDispatchWhenReady(this);

			// set up a task to run on the game thread to accept the results
			FGraphEventArray Prerequistes;
			Prerequistes.Add(ParallelAnimationEvaluationTask);

			check(!IsValidRef(ParallelBlendPhysicsCompletionTask));
			ParallelBlendPhysicsCompletionTask = TGraphTask<FParallelBlendPhysicsCompletionTask>::CreateTask(&Prerequistes).ConstructAndDispatchWhenReady(this);

			ThisTickFunction.GetCompletionHandle()->DontCompleteUntil(ParallelBlendPhysicsCompletionTask);
		}
		else
		{
			PRAGMA_DISABLE_DEPRECATION_WARNINGS
			PerformBlendPhysicsBones(RequiredBones, GetEditableComponentSpaceTransforms(), BoneSpaceTransforms);
			PRAGMA_ENABLE_DEPRECATION_WARNINGS
			FinalizeAnimationUpdate();
		}
	}
}

DECLARE_CYCLE_STAT(TEXT("STAT_FinalizeAnimationUpdate_UpdateChildTransforms"), STAT_FinalizeAnimationUpdate_UpdateChildTransforms, STATGROUP_Anim);
DECLARE_CYCLE_STAT(TEXT("STAT_FinalizeAnimationUpdate_UpdateOverlaps"), STAT_FinalizeAnimationUpdate_UpdateOverlaps, STATGROUP_Anim);
DECLARE_CYCLE_STAT(TEXT("STAT_FinalizeAnimationUpdate_UpdateBounds"), STAT_FinalizeAnimationUpdate_UpdateBounds, STATGROUP_Anim);

void USkeletalMeshComponent::FinalizeAnimationUpdate()
{
	SCOPE_CYCLE_COUNTER(STAT_FinalizeAnimationUpdate);
	
	// Flip bone buffer and send 'post anim' notification
	FinalizeBoneTransform();

	if(!bSimulationUpdatesChildTransforms || !IsSimulatingPhysics() )	//If we simulate physics the call to MoveComponent already updates the children transforms. If we are confident that animation will not be needed this can be skipped. TODO: this should be handled at the scene component layer
	{
		SCOPE_CYCLE_COUNTER(STAT_FinalizeAnimationUpdate_UpdateChildTransforms);

		// Update Child Transform - The above function changes bone transform, so will need to update child transform
		// But only children attached to us via a socket.
		if (bHasSocketAttachments)
		{
			UpdateChildTransforms(EUpdateTransformFlags::OnlyUpdateIfUsingSocket);
		}
	}

	if(bUpdateOverlapsOnAnimationFinalize)
	{
		SCOPE_CYCLE_COUNTER(STAT_FinalizeAnimationUpdate_UpdateOverlaps);

		// animation often change overlap. 
		UpdateOverlaps();
	}

	// update bounds
	if(bSkipBoundsUpdateWhenInterpolating)
	{
		if(AnimEvaluationContext.bDoEvaluation)
		{
			SCOPE_CYCLE_COUNTER(STAT_FinalizeAnimationUpdate_UpdateBounds);
			// Cached local bounds are now out of date
			InvalidateCachedBounds();

			UpdateBounds();
		}
	}
	else
	{
		SCOPE_CYCLE_COUNTER(STAT_FinalizeAnimationUpdate_UpdateBounds);
		// Cached local bounds are now out of date
		InvalidateCachedBounds();

		UpdateBounds();
	}

	// Need to send new bounds to 
	MarkRenderTransformDirty();

	// New bone positions need to be sent to render thread
	MarkRenderDynamicDataDirty();

	// If we have any Follower Components, they need to be refreshed as well.
	RefreshFollowerComponents();
}

void USkeletalMeshComponent::CompleteParallelBlendPhysics()
{
	SwapEvaluationContextBuffers();

	FinalizeAnimationUpdate();

	ParallelAnimationEvaluationTask.SafeRelease();
	ParallelBlendPhysicsCompletionTask.SafeRelease();
}

// NOTE: See GatherActorsAndTransforms in PhysScene_Chaos.cpp where this code is cloned for deferred mode.
// @todo(chaos): merge required deferred functionality back into USkeletalMeshComponent
void USkeletalMeshComponent::UpdateKinematicBonesToAnim(const TArray<FTransform>& InSpaceBases, ETeleportType Teleport, bool bNeedsSkinning, EAllowKinematicDeferral DeferralAllowed)
{
	SCOPE_CYCLE_COUNTER(STAT_UpdateRBBones);

	// Double check that the physics state has been created.
	// If there's no physics state, we can't do anything.
	if (!IsPhysicsStateCreated())
	{
		return;
	}

	// This below code produces some interesting result here
	// - below codes update physics data, so if you don't update pose, the physics won't have the right result
	// - but if we just update physics bone without update current pose, it will have stale data
	// If desired, pass the animation data to the physics joints so they can be used by motors.
	// See if we are going to need to update kinematics
	const bool bUpdateKinematics = (KinematicBonesUpdateType != EKinematicBonesUpdateToPhysics::SkipAllBones);
	const bool bTeleport = Teleport == ETeleportType::TeleportPhysics;
	// If desired, update physics bodies associated with skeletal mesh component to match.
	if(!bUpdateKinematics && !(bTeleport && IsAnySimulatingPhysics()))
	{
		// nothing to do 
		return;
	}

	// Get the scene, and do nothing if we can't get one.
	FPhysScene* PhysScene = nullptr;
	UWorld* World = GetWorld();
	if (World != nullptr)
	{
		PhysScene = GetWorld()->GetPhysicsScene();
	}

	if(PhysScene == nullptr)
	{
		return;
	}

	const FTransform& CurrentLocalToWorld = GetComponentTransform();

#if !(UE_BUILD_SHIPPING)
	// Gracefully handle NaN
	if(CurrentLocalToWorld.ContainsNaN())
	{
		logOrEnsureNanError(TEXT("USkeletalMeshComponent::UpdateKinematicBonesToAnim: CurrentLocalToWorld contains NaN, aborting."));
		return;
	}
#endif

	// If we are only using bodies for physics, don't need to move them right away, can defer until simulation (unless told not to)
	if (DeferralAllowed == EAllowKinematicDeferral::AllowDeferral && (bDeferKinematicBoneUpdate || BodyInstance.GetCollisionEnabled() == ECollisionEnabled::PhysicsOnly))
	{
		if (PhysScene->MarkForPreSimKinematicUpdate(this, Teleport, bNeedsSkinning))
		{
			return;
		}
	}

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	// If desired, draw the skeleton at the point where we pass it to the physics.
	if (bShowPrePhysBones && GetSkeletalMeshAsset() && InSpaceBases.Num() == GetSkeletalMeshAsset()->GetRefSkeleton().GetNum())
	{
		if (ULineBatchComponent* LineBatcher = World->GetLineBatcher(UWorld::ELineBatcherType::World))
		{
			for (int32 i = 1; i<InSpaceBases.Num(); i++)
			{
				FVector ThisPos = CurrentLocalToWorld.TransformPosition(InSpaceBases[i].GetLocation());

				int32 ParentIndex = GetSkeletalMeshAsset()->GetRefSkeleton().GetParentIndex(i);
				FVector ParentPos = CurrentLocalToWorld.TransformPosition(InSpaceBases[ParentIndex].GetLocation());

				LineBatcher->DrawLine(ThisPos, ParentPos, AnimSkelDrawColor, SDPG_Foreground);
			}
		}
	}
#endif

	// warn if it has non-uniform scale
	const FVector& MeshScale3D = CurrentLocalToWorld.GetScale3D();
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	if( !MeshScale3D.IsUniform() )
	{
		UE_LOGF(LogPhysics, Log, "USkeletalMeshComponent::UpdateKinematicBonesToAnim : Non-uniform scale factor (%ls) can cause physics to mismatch for %ls  SkelMesh: %ls", *MeshScale3D.ToString(), *GetFullName(), GetSkeletalMeshAsset() ? *GetSkeletalMeshAsset()->GetFullName() : TEXT("NULL"));
	}
#endif


	if (bEnablePerPolyCollision == false)
	{
		const UPhysicsAsset* const PhysicsAsset = GetPhysicsAsset();
		if (PhysicsAsset && GetSkeletalMeshAsset() && Bodies.Num() > 0)
		{
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
			if (!ensureMsgf(PhysicsAsset->SkeletalBodySetups.Num() == Bodies.Num(), TEXT("Mesh (%s) has PhysicsAsset(%s), and BodySetup(%d) and Bodies(%d) don't match"),
						*GetSkeletalMeshAsset()->GetName(), *PhysicsAsset->GetName(), PhysicsAsset->SkeletalBodySetups.Num(), Bodies.Num()))
			{
				return;
			}
#endif
			const int32 NumComponentSpaceTransforms = GetNumComponentSpaceTransforms();
			const int32 NumBodies = Bodies.Num();

			// Lock the scenes we need (flags set in InitArticulated)
			FPhysicsCommand::ExecuteWrite(this, [&]()
			{
				// Iterate over each body
				for(int32 i = 0; i < NumBodies; i++)
				{
					FBodyInstance* BodyInst = Bodies[i];
					if (!ensure(BodyInst))
					{
						continue;
					}
					FPhysicsActorHandle ActorHandle = BodyInst->GetPhysicsActor();

					if (FPhysicsInterface::IsValid(ActorHandle) &&
						!FPhysicsInterface::IsStatic(ActorHandle) &&
						(bTeleport || !BodyInst->IsInstanceSimulatingPhysics()))	//If we have a body and it's kinematic, or we are teleporting a simulated body
					{
						const int32 BoneIndex = BodyInst->InstanceBoneIndex;

						// If we could not find it - warn.
						if(BoneIndex == INDEX_NONE || BoneIndex >= NumComponentSpaceTransforms)
						{
							FName BodyName = "UNKNOWN";
							if (PhysicsAsset->SkeletalBodySetups.IsValidIndex(i))
							{
								BodyName = PhysicsAsset->SkeletalBodySetups[i]->BoneName;
							}
							UE_LOGF(LogPhysics, Log, "UpdateRBBones: WARNING: Failed to find bone '%ls' need by PhysicsAsset '%ls' in SkeletalMesh '%ls'.", *BodyName.ToString(), *PhysicsAsset->GetName(), *GetSkeletalMeshAsset()->GetName());
						}
						else
						{
							if (BoneIndex >= InSpaceBases.Num())
							{
								FName BodyName = "UNKNOWN";
								if (PhysicsAsset->SkeletalBodySetups.IsValidIndex(i))
								{
									BodyName = PhysicsAsset->SkeletalBodySetups[i]->BoneName;
								}
								UE_LOGF(LogPhysics, Warning, "BoneIndex %d out of range of SpaceBases (Size %d) on PhysicsAsset '%ls' in SkeletalMesh '%ls' for bone '%ls'", BoneIndex, InSpaceBases.Num(), *PhysicsAsset->GetName(), *GetSkeletalMeshAsset()->GetName(), *BodyName.ToString());
								continue;
							}

							// update bone transform to world
							const FTransform BoneTransform = InSpaceBases[BoneIndex] * CurrentLocalToWorld;
							if(!BoneTransform.IsValid())
							{
								FName BodyName = "UNKNOWN";
								if (PhysicsAsset->SkeletalBodySetups.IsValidIndex(i))
								{
									BodyName = PhysicsAsset->SkeletalBodySetups[i]->BoneName;
								}

								UE_LOGF(LogPhysics, Warning, "UpdateKinematicBonesToAnim: Trying to set transform with bad data %ls on PhysicsAsset '%ls' in SkeletalMesh '%ls' for bone '%ls'", *BoneTransform.ToHumanReadableString(), *PhysicsAsset->GetName(), *GetSkeletalMeshAsset()->GetName(), *BodyName.ToString());
								BoneTransform.DiagnosticCheck_IsValid();	//In special nan mode we want to actually ensure

								continue;
							}

							// If not teleporting (must be kinematic) set kinematic target
							if(!bTeleport)
							{
								PhysScene->SetKinematicTarget_AssumesLocked(BodyInst, BoneTransform, true);
							}
							// Otherwise, set global pose
							else
							{
								FPhysicsInterface::SetGlobalPose_AssumesLocked(ActorHandle, BoneTransform);
							}

							if(!PhysicsAsset->SkeletalBodySetups[i]->bSkipScaleFromAnimation)
							{
								// now update scale
								// if uniform, we'll use BoneTranform
								if(MeshScale3D.IsUniform())
								{
									// @todo should we update scale when it's simulated?
									BodyInst->UpdateBodyScale(BoneTransform.GetScale3D());
								}
								else
								{
									// @note When you have non-uniform scale on mesh base,
									// hierarchical bone transform can update scale too often causing performance issue
									// So we just use mesh scale for all bodies when non-uniform
									// This means physics representation won't be accurate, but
									// it is performance friendly by preventing too frequent physics update
									BodyInst->UpdateBodyScale(MeshScale3D);
								}
							}
						}
					}
					else
					{
						//make sure you have physics weight or blendphysics on, otherwise, you'll have inconsistent representation of bodies
						// @todo make this to be kismet log? But can be too intrusive
						if(!bBlendPhysics && BodyInst->PhysicsBlendWeight <= 0.f && BodyInst->BodySetup.IsValid())
						{
							//It's not clear whether this should be a warning. There are certainly cases where you interpolate the blend weight towards 0. The blend feature needs some work which will probably change this in the future.
							//Making it Verbose for now
							UE_LOGF(LogPhysics, Verbose, "%ls(Mesh %ls, PhysicsAsset %ls, Bone %ls) is simulating, but no blending. ",
								*GetName(), *GetNameSafe(GetSkeletalMeshAsset()), *GetNameSafe(PhysicsAsset), *BodyInst->BodySetup.Get()->BoneName.ToString());
						}
					}
				}
			});
		}
	}
	else
	{
		//per poly update requires us to update all vertex positions
		if (MeshObject)
		{
			if (bNeedsSkinning)
			{
				const FSkeletalMeshLODRenderData& LODData = MeshObject->GetSkeletalMeshRenderData().LODRenderData[0];
				FSkinWeightVertexBuffer& SkinWeightBuffer = *GetSkinWeightBuffer(0);
				TArray<FMatrix44f> RefToLocals;
				TArray<FVector3f> NewPositions;
				#if 1
				{
					SCOPE_CYCLE_COUNTER(STAT_SkinPerPolyVertices);
					CacheRefToLocalMatrices(RefToLocals);
					ComputeSkinnedPositions(this, NewPositions, RefToLocals, LODData, SkinWeightBuffer);
				}
				#else	//keep old way around for now - useful for comparing performance
				{
					NewPositions.AddUninitialized(LODData.GetNumVertices());
					{
						SCOPE_CYCLE_COUNTER(STAT_SkinPerPolyVertices);
						for (uint32 VertIndex = 0; VertIndex < LODData.GetNumVertices(); ++VertIndex)
						{
							NewPositions[VertIndex] = GetSkinnedVertexPosition(this, VertIndex, LODData, SkinWeightBuffer);
						}
					}
				}
				#endif
				BodyInstance.UpdateTriMeshVertices(UE::LWC::ConvertArrayType<FVector>(NewPositions));
			}
			
			BodyInstance.SetBodyTransform(CurrentLocalToWorld, Teleport);
		}
	}


}



void USkeletalMeshComponent::UpdateRBJointMotors()
{
	SCOPE_CYCLE_COUNTER(STAT_UpdateRBJoints);

	// moved this flag to here, so that
	// you can call it but still respect the flag
	if( !bUpdateJointsFromAnimation )
	{
		return;
	}

	const UPhysicsAsset* const PhysicsAsset = GetPhysicsAsset();
	if(PhysicsAsset && Constraints.Num() > 0 && GetSkeletalMeshAsset())
	{
		check( PhysicsAsset->ConstraintSetup.Num() == Constraints.Num() );


		// Iterate over the constraints.
		for(int32 i=0; i<Constraints.Num(); i++)
		{
			UPhysicsConstraintTemplate* CS = PhysicsAsset->ConstraintSetup[i];
			FConstraintInstance* CI = Constraints[i];

			FName JointChildBoneName = CS->DefaultInstance.GetChildBoneName();
			int32 BoneIndex = GetSkeletalMeshAsset()->GetRefSkeleton().FindBoneIndex(JointChildBoneName);

			// If we found this bone, and a visible bone that is not the root, and its joint is motorised in some way..
			if( (BoneIndex != INDEX_NONE) && (BoneIndex != 0) &&
				(GetBoneVisibilityStates()[BoneIndex] == BVS_Visible) &&
				(CI->IsAngularOrientationDriveEnabled()) )
			{
				PRAGMA_DISABLE_DEPRECATION_WARNINGS
				check(BoneIndex < BoneSpaceTransforms.Num());

				// If we find the joint - get the local-space animation between this bone and its parent.
				FQuat LocalQuat = BoneSpaceTransforms[BoneIndex].GetRotation();
				PRAGMA_ENABLE_DEPRECATION_WARNINGS
				FQuatRotationTranslationMatrix LocalRot(LocalQuat, FVector::ZeroVector);

				// We loop from the graphics parent bone up to the bone that has the body which the joint is attached to, to calculate the relative transform.
				// We need this to compensate for welding, where graphics and physics parents may not be the same.
				FMatrix ControlBodyToParentBoneTM = FMatrix::Identity;

				int32 TestBoneIndex = GetSkeletalMeshAsset()->GetRefSkeleton().GetParentIndex(BoneIndex); // This give the 'graphics' parent of this bone
				bool bFoundControlBody = (GetSkeletalMeshAsset()->GetRefSkeleton().GetBoneName(TestBoneIndex) == CS->DefaultInstance.ConstraintBone2); // ConstraintBone2 is the 'physics' parent of this joint.

				while(!bFoundControlBody)
				{
					PRAGMA_DISABLE_DEPRECATION_WARNINGS
					// Abort if we find a bone scaled to zero.
					const FVector Scale3D = BoneSpaceTransforms[TestBoneIndex].GetScale3D();
					PRAGMA_ENABLE_DEPRECATION_WARNINGS
					const float ScaleSum = Scale3D.X + Scale3D.Y + Scale3D.Z;
					if(ScaleSum < UE_KINDA_SMALL_NUMBER)
					{
						break;
					}

					PRAGMA_DISABLE_DEPRECATION_WARNINGS
					// Add the current animated local transform into the overall controlling body->parent bone TM
					FMatrix RelTM = BoneSpaceTransforms[TestBoneIndex].ToMatrixNoScale();
					PRAGMA_ENABLE_DEPRECATION_WARNINGS
					RelTM.SetOrigin(FVector::ZeroVector);
					ControlBodyToParentBoneTM = ControlBodyToParentBoneTM * RelTM;

					// Move on to parent
					TestBoneIndex = GetSkeletalMeshAsset()->GetRefSkeleton().GetParentIndex(TestBoneIndex);

					// If we are at the root - bail out.
					if(TestBoneIndex == 0)
					{
						break;
					}

					// See if this is the controlling body
					bFoundControlBody = (GetSkeletalMeshAsset()->GetRefSkeleton().GetBoneName(TestBoneIndex) == CS->DefaultInstance.ConstraintBone2);
				}

				// If after that we didn't find a parent body, we can' do this, so skip.
				if(bFoundControlBody)
				{
					// The animation rotation is between the two bodies. We need to supply the joint with the relative orientation between
					// the constraint ref frames. So we work out each body->joint transform

					FMatrix Body1TM = CS->DefaultInstance.GetRefFrame(EConstraintFrame::Frame1).ToMatrixNoScale();
					Body1TM.SetOrigin(FVector::ZeroVector);
					FMatrix Body1TMInv = Body1TM.InverseFast();

					FMatrix Body2TM = CS->DefaultInstance.GetRefFrame(EConstraintFrame::Frame2).ToMatrixNoScale();
					Body2TM.SetOrigin(FVector::ZeroVector);
					FMatrix Body2TMInv = Body2TM.InverseFast();

					FMatrix JointRot = Body1TM * (LocalRot * ControlBodyToParentBoneTM) * Body2TMInv;
					FQuat JointQuat(JointRot);

					// Then pass new quaternion to the joint!
					CI->SetAngularOrientationTarget(JointQuat);
				}
			}
		}
	}
}

