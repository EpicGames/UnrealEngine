// Copyright Epic Games, Inc. All Rights Reserved.


#include "EngineLogs.h"
#include "PhysicsPublic.h"
#include "Trace/Trace.inl"

FPhysCommandHandler::~FPhysCommandHandler()
{
	if(PendingCommands.Num() != 0)
	{
		UE_LOGF(LogPhysics, Warning, "~FPhysCommandHandler() - Pending command list is not empty. %d item remain.", PendingCommands.Num());
	}
}

void FPhysCommandHandler::Flush()
{
	check(IsInGameThread());
	ExecuteCommands();
	PendingCommands.Empty();
}

bool FPhysCommandHandler::HasPendingCommands()
{
	return PendingCommands.Num() > 0;
}

void FPhysCommandHandler::ExecuteCommands()
{
	for (int32 i = 0; i < PendingCommands.Num(); ++i)
	{
		const FPhysPendingCommand& Command = PendingCommands[i];
		switch (Command.CommandType)
		{
		case PhysCommand::ReleasePScene:
		{
			break;
		}
		case PhysCommand::DeleteSimEventCallback:
		{
			break;
		}
		case PhysCommand::DeleteContactModifyCallback:
		{
			break;
		}

		case PhysCommand::DeleteCCDContactModifyCallback:
		{
			break;
		}

		case PhysCommand::DeleteCPUDispatcher:
		{
			break;
		}

		case PhysCommand::DeleteMbpBroadphaseCallback:
		{
			break;
		}
		case PhysCommand::Max:	//this is just here because right now all commands are APEX and having a switch with only default is bad
		default:
			check(0);	//Unsupported command
			break;
		}
	}
}

void FPhysCommandHandler::EnqueueCommand(const FPhysPendingCommand& Command)
{
	check(IsInGameThread());
	PendingCommands.Add(Command);
}
