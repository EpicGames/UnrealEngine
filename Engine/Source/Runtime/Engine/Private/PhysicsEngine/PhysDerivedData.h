// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/Guid.h"
#include "EngineDefines.h"
#include "Stats/Stats.h"
#include "PhysDerivedDataPublic.h"
