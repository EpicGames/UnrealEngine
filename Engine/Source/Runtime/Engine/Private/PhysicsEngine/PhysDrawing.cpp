// Copyright Epic Games, Inc. All Rights Reserved.

#include "EngineLogs.h"
#include "RenderUtils.h"
#include "Engine/Engine.h"
#include "Math/RotationMatrix.h"
#include "PrimitiveDrawingUtils.h"
#include "SceneView.h"
#include "Materials/MaterialInstanceDynamic.h"
#include "DynamicMeshBuilder.h"
#include "PhysicsEngine/AggregateGeom.h"
#include "PhysicsEngine/LevelSetElem.h"
#include "PhysicsEngine/PhysicsConstraintTemplate.h"
#include "PhysicsEngine/ConvexElem.h"
#include "PhysicsEngine/BoxElem.h"
#include "PhysicsEngine/SphereElem.h"
#include "PhysicsEngine/SphylElem.h"
#include "PhysicsEngine/PhysicsAsset.h"
#include "PhysicsEngine/SkeletalBodySetup.h"
#include "PhysicsEngine/SkinnedLevelSetElem.h"
#include "PhysicsEngine/SkinnedTriangleMeshElem.h"
#include "PhysicsEngine/TaperedCapsuleElem.h"
#include "ReferenceSkeleton.h"
#include "StaticMeshResources.h"
#include "Chaos/Levelset.h"
#include "Chaos/SkinnedTriangleMesh.h"
#include "Chaos/UniformGrid.h"
#include "Chaos/WeightedLatticeImplicitObject.h"
#include "Chaos/PBDJointConstraintUtilities.h"

//UE_DISABLE_OPTIMIZATION
#include "Misc/ScopeLock.h"

static const int32 DrawCollisionSides = 32;
static const int32 DrawConeLimitSides = 40;

static const float DebugJointPosSize = 5.0f;
static const float DebugJointAxisSize = 20.0f;

static const float JointRenderThickness = 0.1f;
static const float UnselectedJointRenderSize = 4.f;
static const float SelectedJointRenderSize = 10.f;
static const float LimitRenderSize = 0.16f;

static const float ConstraintPointSize = 4.0f;
static const float ConstraintLimitViolatedPointSize = 6.0f;

static const FColor JointUnselectedColor(255, 0, 255);
static const FColor JointRed(FColor::Red);
static const FColor JointGreen(FColor::Green);
static const FColor JointBlue(FColor::Blue);

static const FColor	JointLimitColor(FColor::Green);
static const FColor	JointRefColor(FColor::Yellow);
static const FColor JointLockedColor(255,128,10);

static const FColor JointLimitsViolatedColor(FColor::Purple);

static int SkinnedLatticeBoneWeight = -1;
static FAutoConsoleVariableRef CVarClothVizDrawSkinnedLattice(TEXT("p.PhysDrawing.SkinnedLatticeBoneWeight"), SkinnedLatticeBoneWeight, TEXT("Draw skinned lattice bone weight. -1 = all lattice points"));

/////////////////////////////////////////////////////////////////////////////////////
// FKSphereElem
/////////////////////////////////////////////////////////////////////////////////////

void FKSphereElem::DrawElemWire(class FPrimitiveDrawInterface* PDI, const FTransform& ElemTM, float Scale, const FColor Color) const
{
	DrawElemWire(PDI, ElemTM, FVector(Scale), Color);
}

void FKSphereElem::DrawElemSolid(class FPrimitiveDrawInterface* PDI, const FTransform& ElemTM, float Scale, const FMaterialRenderProxy* MaterialRenderProxy) const
{
	DrawElemSolid(PDI, ElemTM, FVector(Scale), MaterialRenderProxy);
}

void FKSphereElem::DrawElemWire(class FPrimitiveDrawInterface* PDI, const FTransform& ElemTM, const FVector& Scale3D, const FColor Color) const
{
	FVector ElemCenter = ElemTM.GetLocation();
	FVector X = ElemTM.GetScaledAxis(EAxis::X);
	FVector Y = ElemTM.GetScaledAxis(EAxis::Y);
	FVector Z = ElemTM.GetScaledAxis(EAxis::Z);

	const float ScaleRadius = Scale3D.GetAbsMin();

	DrawCircle(PDI, ElemCenter, X, Y, Color, ScaleRadius*Radius, DrawCollisionSides, SDPG_World);
	DrawCircle(PDI, ElemCenter, X, Z, Color, ScaleRadius*Radius, DrawCollisionSides, SDPG_World);
	DrawCircle(PDI, ElemCenter, Y, Z, Color, ScaleRadius*Radius, DrawCollisionSides, SDPG_World);
}


void FKSphereElem::DrawElemSolid(class FPrimitiveDrawInterface* PDI, const FTransform& ElemTM, const FVector& Scale3D, const FMaterialRenderProxy* MaterialRenderProxy) const
{
	DrawSphere(PDI, ElemTM.GetLocation(), FRotator::ZeroRotator, FVector(this->Radius * Scale3D.GetAbsMin()), DrawCollisionSides, DrawCollisionSides / 2, MaterialRenderProxy, SDPG_World);
}

void FKSphereElem::GetElemSolid(const FTransform& ElemTM, const FVector& Scale3D, const FMaterialRenderProxy* MaterialRenderProxy, int32 ViewIndex, FMeshElementCollector& Collector) const
{
	GetSphereMesh(ElemTM.GetLocation(), FVector(this->Radius * Scale3D.GetAbsMin()), DrawCollisionSides, DrawCollisionSides / 2, MaterialRenderProxy, SDPG_World, false, ViewIndex, Collector);
}

/////////////////////////////////////////////////////////////////////////////////////
// FKBoxElem
/////////////////////////////////////////////////////////////////////////////////////

void FKBoxElem::DrawElemWire(class FPrimitiveDrawInterface* PDI, const FTransform& ElemTM, float Scale, const FColor Color) const
{
	DrawElemWire(PDI, ElemTM, FVector(Scale), Color);
}

void FKBoxElem::DrawElemSolid(class FPrimitiveDrawInterface* PDI, const FTransform& ElemTM, float Scale, const FMaterialRenderProxy* MaterialRenderProxy) const
{
	DrawElemSolid(PDI, ElemTM, FVector(Scale), MaterialRenderProxy);
}

void FKBoxElem::DrawElemWire(class FPrimitiveDrawInterface* PDI, const FTransform& ElemTM, const FVector& Scale3D, const FColor Color) const
{
	FVector	B[2], P, Q, Radii;

	// X,Y,Z member variables are LENGTH not RADIUS
	Radii.X = Scale3D.X * 0.5f * X;
	Radii.Y = Scale3D.Y * 0.5f * Y;
	Radii.Z = Scale3D.Z * 0.5f * Z;

	B[0] = Radii; // max
	B[1] = -1.0f * Radii; // min

	for (int32 i = 0; i < 2; i++)
	{
		for (int32 j = 0; j < 2; j++)
		{
			P.X = B[i].X; Q.X = B[i].X;
			P.Y = B[j].Y; Q.Y = B[j].Y;
			P.Z = B[0].Z; Q.Z = B[1].Z;
			PDI->DrawLine(ElemTM.TransformPosition(P), ElemTM.TransformPosition(Q), Color, SDPG_World);

			P.Y = B[i].Y; Q.Y = B[i].Y;
			P.Z = B[j].Z; Q.Z = B[j].Z;
			P.X = B[0].X; Q.X = B[1].X;
			PDI->DrawLine(ElemTM.TransformPosition(P), ElemTM.TransformPosition(Q), Color, SDPG_World);

			P.Z = B[i].Z; Q.Z = B[i].Z;
			P.X = B[j].X; Q.X = B[j].X;
			P.Y = B[0].Y; Q.Y = B[1].Y;
			PDI->DrawLine(ElemTM.TransformPosition(P), ElemTM.TransformPosition(Q), Color, SDPG_World);
		}
	}
}

void FKBoxElem::DrawElemSolid(class FPrimitiveDrawInterface* PDI, const FTransform& ElemTM, const FVector& Scale3D, const FMaterialRenderProxy* MaterialRenderProxy) const
{
	DrawBox(PDI, ElemTM.ToMatrixWithScale(), Scale3D * 0.5f * FVector(X, Y, Z), MaterialRenderProxy, SDPG_World);
}

void FKBoxElem::GetElemSolid(const FTransform& ElemTM, const FVector& Scale3D, const FMaterialRenderProxy* MaterialRenderProxy, int32 ViewIndex, FMeshElementCollector& Collector) const
{
	GetBoxMesh(ElemTM.ToMatrixWithScale(), Scale3D * 0.5f * FVector(X, Y, Z), MaterialRenderProxy, SDPG_World, ViewIndex, Collector);
}

/////////////////////////////////////////////////////////////////////////////////////
// FKSphylElem
/////////////////////////////////////////////////////////////////////////////////////

static void DrawHalfCircle(FPrimitiveDrawInterface* PDI, const FVector& Base, const FVector& X, const FVector& Y, const FColor Color, float Radius)
{
	float	AngleDelta = 2.0f * (float)UE_PI / ((float)DrawCollisionSides);
	FVector	LastVertex = Base + X * Radius;

	for(int32 SideIndex = 0; SideIndex < (DrawCollisionSides/2); SideIndex++)
	{
		FVector	Vertex = Base + (X * FMath::Cos(AngleDelta * (SideIndex + 1)) + Y * FMath::Sin(AngleDelta * (SideIndex + 1))) * Radius;
		PDI->DrawLine(LastVertex, Vertex, Color, SDPG_World);
		LastVertex = Vertex;
	}	
}

void FKSphylElem::DrawElemWire(class FPrimitiveDrawInterface* PDI, const FTransform& ElemTM, float Scale, const FColor Color) const
{
	DrawElemWire(PDI, ElemTM, FVector(Scale), Color);
}

void FKSphylElem::DrawElemSolid(class FPrimitiveDrawInterface* PDI, const FTransform& ElemTM, float Scale, const FMaterialRenderProxy* MaterialRenderProxy) const
{
	DrawElemSolid(PDI, ElemTM, FVector(Scale), MaterialRenderProxy);
}

void FKSphylElem::DrawElemWire(class FPrimitiveDrawInterface* PDI, const FTransform& ElemTM, const FVector& Scale3D, const FColor Color) const
{
	const FVector Origin = ElemTM.GetLocation();
	const FVector XAxis = ElemTM.GetScaledAxis(EAxis::X);
	const FVector YAxis = ElemTM.GetScaledAxis(EAxis::Y);
	const FVector ZAxis = ElemTM.GetScaledAxis(EAxis::Z);
	const float ScaledHalfLength = GetScaledCylinderLength(Scale3D) * .5f;
	const float ScaledRadius = GetScaledRadius(Scale3D);
	
	// Draw top and bottom circles
	const FVector TopEnd = Origin + (ScaledHalfLength * ZAxis);
	const FVector BottomEnd = Origin - (ScaledHalfLength * ZAxis);

	DrawCircle(PDI, TopEnd, XAxis, YAxis, Color, ScaledRadius, DrawCollisionSides, SDPG_World);
	DrawCircle(PDI, BottomEnd, XAxis, YAxis, Color, ScaledRadius, DrawCollisionSides, SDPG_World);

	// Draw domed caps
	DrawHalfCircle(PDI, TopEnd, YAxis, ZAxis, Color, ScaledRadius);
	DrawHalfCircle(PDI, TopEnd, XAxis, ZAxis, Color, ScaledRadius);

	const FVector NegZAxis = -ZAxis;

	DrawHalfCircle(PDI, BottomEnd, YAxis, NegZAxis, Color, ScaledRadius);
	DrawHalfCircle(PDI, BottomEnd, XAxis, NegZAxis, Color,ScaledRadius);

	// Draw connecty lines
	PDI->DrawLine(TopEnd + ScaledRadius*XAxis, BottomEnd + ScaledRadius*XAxis, Color, SDPG_World);
	PDI->DrawLine(TopEnd - ScaledRadius*XAxis, BottomEnd - ScaledRadius*XAxis, Color, SDPG_World);
	PDI->DrawLine(TopEnd + ScaledRadius*YAxis, BottomEnd + ScaledRadius*YAxis, Color, SDPG_World);
	PDI->DrawLine(TopEnd - ScaledRadius*YAxis, BottomEnd - ScaledRadius*YAxis, Color, SDPG_World);
}

void FKSphylElem::GetElemSolid(const FTransform& ElemTM, const FVector& Scale3D, const FMaterialRenderProxy* MaterialRenderProxy, int32 ViewIndex, FMeshElementCollector& Collector) const
{
	const FVector Scale3DAbs = Scale3D.GetAbs();
	const float ScaleRadius = FMath::Max(Scale3DAbs.X, Scale3DAbs.Y);
	const float ScaleLength = Scale3DAbs.Z;

	const int32 NumSides = DrawCollisionSides;
	const int32 NumRings = (DrawCollisionSides / 2) + 1;

	// The first/last arc are on top of each other.
	const int32 NumVerts = (NumSides + 1) * (NumRings + 1);
	FDynamicMeshVertex* Verts = (FDynamicMeshVertex*)FMemory::Malloc(NumVerts * sizeof(FDynamicMeshVertex));

	// Calculate verts for one arc
	FDynamicMeshVertex* ArcVerts = (FDynamicMeshVertex*)FMemory::Malloc((NumRings + 1) * sizeof(FDynamicMeshVertex));

	for (int32 RingIdx = 0; RingIdx < NumRings + 1; RingIdx++)
	{
		FDynamicMeshVertex* ArcVert = &ArcVerts[RingIdx];

		float Angle;
		float ZOffset;
		if (RingIdx <= DrawCollisionSides / 4)
		{
			Angle = ((float)RingIdx / (NumRings - 1)) * UE_PI;
			ZOffset = 0.5 * ScaleLength * Length;
		}
		else
		{
			Angle = ((float)(RingIdx - 1) / (NumRings - 1)) * UE_PI;
			ZOffset = -0.5 * ScaleLength * Length;
		}

		// Note- unit sphere, so position always has mag of one. We can just use it for normal!		
		FVector SpherePos;
		SpherePos.X = 0.0f;
		SpherePos.Y = ScaleRadius * Radius * FMath::Sin(Angle);
		SpherePos.Z = ScaleRadius * Radius * FMath::Cos(Angle);

		ArcVert->Position = FVector3f(SpherePos + FVector(0, 0, ZOffset));

		ArcVert->SetTangents(
			FVector3f(1, 0, 0),
			FVector3f(0.0f, -SpherePos.Z, SpherePos.Y),
			(FVector3f)SpherePos
			);

		ArcVert->TextureCoordinate[0].X = 0.0f;
		ArcVert->TextureCoordinate[0].Y = ((float)RingIdx / NumRings);
	}

	// Then rotate this arc NumSides+1 times.
	for (int32 SideIdx = 0; SideIdx < NumSides + 1; SideIdx++)
	{
		const FRotator3f ArcRotator(0, 360.f * ((float)SideIdx / NumSides), 0);
		const FRotationMatrix44f ArcRot(ArcRotator);
		const float XTexCoord = ((float)SideIdx / NumSides);

		for (int32 VertIdx = 0; VertIdx < NumRings + 1; VertIdx++)
		{
			int32 VIx = (NumRings + 1)*SideIdx + VertIdx;

			Verts[VIx].Position = ArcRot.TransformPosition(ArcVerts[VertIdx].Position);

			Verts[VIx].SetTangents(
				ArcRot.TransformVector(ArcVerts[VertIdx].TangentX.ToFVector3f()),
				ArcRot.TransformVector(ArcVerts[VertIdx].GetTangentY()),
				ArcRot.TransformVector(ArcVerts[VertIdx].TangentZ.ToFVector3f())
				);

			Verts[VIx].TextureCoordinate[0].X = XTexCoord;
			Verts[VIx].TextureCoordinate[0].Y = ArcVerts[VertIdx].TextureCoordinate[0].Y;
		}
	}

	FDynamicMeshBuilder MeshBuilder(Collector.GetFeatureLevel());
	{
		// Add all of the vertices to the mesh.
		for (int32 VertIdx = 0; VertIdx < NumVerts; VertIdx++)
		{
			MeshBuilder.AddVertex(Verts[VertIdx]);
		}

		// Add all of the triangles to the mesh.
		for (int32 SideIdx = 0; SideIdx < NumSides; SideIdx++)
		{
			const int32 a0start = (SideIdx + 0) * (NumRings + 1);
			const int32 a1start = (SideIdx + 1) * (NumRings + 1);

			for (int32 RingIdx = 0; RingIdx < NumRings; RingIdx++)
			{
				MeshBuilder.AddTriangle(a0start + RingIdx + 0, a1start + RingIdx + 0, a0start + RingIdx + 1);
				MeshBuilder.AddTriangle(a1start + RingIdx + 0, a1start + RingIdx + 1, a0start + RingIdx + 1);
			}
		}

	}
	MeshBuilder.GetMesh(ElemTM.ToMatrixWithScale(), MaterialRenderProxy, SDPG_World, false, false, ViewIndex, Collector);

	FMemory::Free(Verts);
	FMemory::Free(ArcVerts);
}

void FKSphylElem::DrawElemSolid(FPrimitiveDrawInterface* PDI, const FTransform& ElemTM, const FVector& Scale3D, const FMaterialRenderProxy* MaterialRenderProxy) const
{
	const FVector Scale3DAbs = Scale3D.GetAbs();
	const float ScaleRadius = FMath::Max(Scale3DAbs.X, Scale3DAbs.Y);
	const float ScaleLength = Scale3DAbs.Z;

	const int32 NumSides = DrawCollisionSides;
	const int32 NumRings = (DrawCollisionSides / 2) + 1;

	// The first/last arc are on top of each other.
	const int32 NumVerts = (NumSides + 1) * (NumRings + 1);
	FDynamicMeshVertex* Verts = (FDynamicMeshVertex*)FMemory::Malloc(NumVerts * sizeof(FDynamicMeshVertex));

	// Calculate verts for one arc
	FDynamicMeshVertex* ArcVerts = (FDynamicMeshVertex*)FMemory::Malloc((NumRings + 1) * sizeof(FDynamicMeshVertex));

	for (int32 RingIdx = 0; RingIdx < NumRings + 1; RingIdx++)
	{
		FDynamicMeshVertex* ArcVert = &ArcVerts[RingIdx];

		float Angle;
		float ZOffset;
		if (RingIdx <= DrawCollisionSides / 4)
		{
			Angle = ((float)RingIdx / (NumRings - 1)) * UE_PI;
			ZOffset = 0.5 * ScaleLength * Length;
		}
		else
		{
			Angle = ((float)(RingIdx - 1) / (NumRings - 1)) * UE_PI;
			ZOffset = -0.5 * ScaleLength * Length;
		}

		// Note- unit sphere, so position always has mag of one. We can just use it for normal!		
		FVector SpherePos;
		SpherePos.X = 0.0f;
		SpherePos.Y = ScaleRadius * Radius * FMath::Sin(Angle);
		SpherePos.Z = ScaleRadius * Radius * FMath::Cos(Angle);

		ArcVert->Position = FVector3f(SpherePos + FVector(0, 0, ZOffset));

		ArcVert->SetTangents(
			FVector3f(1, 0, 0),
			FVector3f(0.0f, -SpherePos.Z, SpherePos.Y),
			(FVector3f)SpherePos
			);

		ArcVert->TextureCoordinate[0].X = 0.0f;
		ArcVert->TextureCoordinate[0].Y = ((float)RingIdx / NumRings);
	}

	// Then rotate this arc NumSides+1 times.
	for (int32 SideIdx = 0; SideIdx < NumSides + 1; SideIdx++)
	{
		const FRotator3f ArcRotator(0, 360.f * ((float)SideIdx / NumSides), 0);
		const FRotationMatrix44f ArcRot(ArcRotator);
		const float XTexCoord = ((float)SideIdx / NumSides);

		for (int32 VertIdx = 0; VertIdx < NumRings + 1; VertIdx++)
		{
			int32 VIx = (NumRings + 1)*SideIdx + VertIdx;

			Verts[VIx].Position = ArcRot.TransformPosition(ArcVerts[VertIdx].Position);

			Verts[VIx].SetTangents(
				ArcRot.TransformVector(ArcVerts[VertIdx].TangentX.ToFVector3f()),
				ArcRot.TransformVector(ArcVerts[VertIdx].GetTangentY()),
				ArcRot.TransformVector(ArcVerts[VertIdx].TangentZ.ToFVector3f())
				);

			Verts[VIx].TextureCoordinate[0].X = XTexCoord;
			Verts[VIx].TextureCoordinate[0].Y = ArcVerts[VertIdx].TextureCoordinate[0].Y;
		}
	}

	FDynamicMeshBuilder MeshBuilder(PDI->View->GetFeatureLevel());
	{
		// Add all of the vertices to the mesh.
		for (int32 VertIdx = 0; VertIdx < NumVerts; VertIdx++)
		{
			MeshBuilder.AddVertex(Verts[VertIdx]);
		}

		// Add all of the triangles to the mesh.
		for (int32 SideIdx = 0; SideIdx < NumSides; SideIdx++)
		{
			const int32 a0start = (SideIdx + 0) * (NumRings + 1);
			const int32 a1start = (SideIdx + 1) * (NumRings + 1);

			for (int32 RingIdx = 0; RingIdx < NumRings; RingIdx++)
			{
				MeshBuilder.AddTriangle(a0start + RingIdx + 0, a1start + RingIdx + 0, a0start + RingIdx + 1);
				MeshBuilder.AddTriangle(a1start + RingIdx + 0, a1start + RingIdx + 1, a0start + RingIdx + 1);
			}
		}

	}
	MeshBuilder.Draw(PDI, ElemTM.ToMatrixWithScale(), MaterialRenderProxy, SDPG_World);


	FMemory::Free(Verts);
	FMemory::Free(ArcVerts);
}

/////////////////////////////////////////////////////////////////////////////////////
// FKTaperedCapsuleElem
/////////////////////////////////////////////////////////////////////////////////////

void FKTaperedCapsuleElem::DrawTaperedCapsuleSides(FPrimitiveDrawInterface* PDI, const FQuat& CapsuleOrientation, const FVector& InCenter0, const FVector& InCenter1, float InRadius0, float InRadius1, const FColor& Color, bool bInSplitTaperedCylinder)
{
	const FColor HalfTaperedColor = bInSplitTaperedCylinder ? (FLinearColor(Color) * .2f).ToFColor(true) : Color;

	// Draws just the sides of a tapered capsule specified by provided Spheres that can have different radii.  Does not draw the spheres, just the sleeve.
	// Extent geometry endpoints not necessarily coplanar with sphere origins (uses hull horizon)
	// Otherwise uses the great-circle cap assumption.
	const float AngleIncrementCircle = 360.0f / DrawCollisionSides;
	const float AngleIncrementSides = 90.0f; 
	const FVector Separation = InCenter1 - InCenter0;
	const float Distance = Separation.Size();
	if(Separation.IsNearlyZero() || Distance <= FMath::Abs(InRadius0 - InRadius1))
	{
		return;
	}
	float OffsetZ = (InRadius1 - InRadius0) / Distance;
	float ScaleXY = FMath::Sqrt(1.0f - FMath::Square(OffsetZ));

	FVector StartVertex = CapsuleOrientation.RotateVector(FVector(0, ScaleXY, OffsetZ));
	FVector VertexPrevious = StartVertex;
	for(float Angle = AngleIncrementCircle; Angle <= 180.0f; Angle += AngleIncrementCircle)  // iterate over unit circle about capsule's major axis (which is orientation.AxisZ)
	{
		FVector VertexCurrent = CapsuleOrientation.RotateVector(FVector(FMath::Sin(FMath::DegreesToRadians(Angle)) * ScaleXY, FMath::Cos(FMath::DegreesToRadians(Angle))*ScaleXY, OffsetZ));
		PDI->DrawTranslucentLine(InCenter0 + VertexPrevious * InRadius0, InCenter0 + VertexCurrent * InRadius0, Color, SDPG_World);  // cap-circle segment on sphere S0
		PDI->DrawTranslucentLine(InCenter1 + VertexPrevious * InRadius1, InCenter1 + VertexCurrent * InRadius1, Color, SDPG_World);  // cap-circle segment on sphere S1
		VertexPrevious = VertexCurrent;
	}

	for (float Angle = 180.f + AngleIncrementCircle; Angle <= 360.0f; Angle += AngleIncrementCircle)  // iterate over unit circle about capsule's major axis (which is orientation.AxisZ)
	{
		FVector VertexCurrent = CapsuleOrientation.RotateVector(FVector(FMath::Sin(FMath::DegreesToRadians(Angle)) * ScaleXY, FMath::Cos(FMath::DegreesToRadians(Angle)) * ScaleXY, OffsetZ));
		PDI->DrawTranslucentLine(InCenter0 + VertexPrevious * InRadius0, InCenter0 + VertexCurrent * InRadius0, HalfTaperedColor, SDPG_World);  // cap-circle segment on sphere S0
		PDI->DrawTranslucentLine(InCenter1 + VertexPrevious * InRadius1, InCenter1 + VertexCurrent * InRadius1, HalfTaperedColor, SDPG_World);  // cap-circle segment on sphere S1
		VertexPrevious = VertexCurrent;
	}

	for (float Angle = 0; Angle <= 180.f; Angle += AngleIncrementSides)  // iterate over unit circle about capsule's major axis (which is orientation.AxisZ)
	{
		FVector VertexCurrent = CapsuleOrientation.RotateVector(FVector(FMath::Sin(FMath::DegreesToRadians(Angle)) * ScaleXY, FMath::Cos(FMath::DegreesToRadians(Angle)) * ScaleXY, OffsetZ));
		PDI->DrawTranslucentLine(InCenter0 + VertexCurrent * InRadius0, InCenter1 + VertexCurrent * InRadius1, Color, SDPG_World);  // capsule side segment between spheres
	}
	for(float Angle = 180.f + AngleIncrementSides; Angle < 360.0f; Angle += AngleIncrementSides)  // iterate over unit circle about capsule's major axis (which is orientation.AxisZ)
	{
		FVector VertexCurrent = CapsuleOrientation.RotateVector(FVector(FMath::Sin(FMath::DegreesToRadians(Angle)) * ScaleXY, FMath::Cos(FMath::DegreesToRadians(Angle)) * ScaleXY, OffsetZ));
		PDI->DrawTranslucentLine(InCenter0 + VertexCurrent  * InRadius0, InCenter1 + VertexCurrent * InRadius1, HalfTaperedColor, SDPG_World);  // capsule side segment between spheres
	}
}

static void DrawWireframeTaperedCapsule(class FPrimitiveDrawInterface* PDI, const FColor& Color, const FQuat& CapsuleOrientation, const FVector& Origin, const FVector& XAxis, const FVector& YAxis, const FVector& ZAxis, const float ScaledHalfLength, const float ScaledRadius0, const float ScaledRadius1, const bool bIsOneSided)
{
	const FColor HalfTaperedColor = bIsOneSided ? (FLinearColor(Color) * .2f).ToFColor(true) : Color;
	// Draw top and bottom circles
	const FVector TopEnd = Origin + (ScaledHalfLength * ZAxis);
	const FVector BottomEnd = Origin - (ScaledHalfLength * ZAxis);

	const FVector Separation = TopEnd - BottomEnd;
	float Distance = Separation.Size();

	if (Separation.IsNearlyZero() || Distance <= FMath::Abs(ScaledRadius0 - ScaledRadius1))
	{
		// Degenerate or one end encompasses the other - draw a sphere
		if (ScaledRadius0 > ScaledRadius1)
		{
			DrawArc(PDI, TopEnd, XAxis, YAxis, -90.f, 90.f, ScaledRadius0, DrawCollisionSides, Color, SDPG_World);
			DrawArc(PDI, TopEnd, XAxis, ZAxis, -90.f, 90.f, ScaledRadius0, DrawCollisionSides, Color, SDPG_World);

			DrawArc(PDI, TopEnd, XAxis, YAxis, 90.f, 270.f, ScaledRadius0, DrawCollisionSides, HalfTaperedColor, SDPG_World);
			DrawArc(PDI, TopEnd, XAxis, ZAxis, 90.f, 270.f, ScaledRadius0, DrawCollisionSides, HalfTaperedColor, SDPG_World);
			DrawCircle(PDI, TopEnd, YAxis, ZAxis, Color, ScaledRadius0, DrawCollisionSides, SDPG_World);
		}
		else
		{
			DrawArc(PDI, BottomEnd, XAxis, YAxis, -90.f, 90.f, ScaledRadius1, DrawCollisionSides, Color, SDPG_World);
			DrawArc(PDI, BottomEnd, XAxis, ZAxis, -90.f, 90.f, ScaledRadius1, DrawCollisionSides, Color, SDPG_World);

			DrawArc(PDI, BottomEnd, XAxis, YAxis, 90.f, 270.f, ScaledRadius1, DrawCollisionSides, HalfTaperedColor, SDPG_World);
			DrawArc(PDI, BottomEnd, XAxis, ZAxis, 90.f, 270.f, ScaledRadius1, DrawCollisionSides, HalfTaperedColor, SDPG_World);
			DrawCircle(PDI, BottomEnd, YAxis, ZAxis, Color, ScaledRadius1, DrawCollisionSides, SDPG_World);
		}
	}
	else
	{
		if (ScaledRadius0 > ScaledRadius1)
		{
			DrawArc(PDI, TopEnd, XAxis, YAxis, -90.f, 90.f, ScaledRadius0, DrawCollisionSides, Color, SDPG_World);
			DrawArc(PDI, TopEnd, XAxis, YAxis, 90.f, 270.f, ScaledRadius0, DrawCollisionSides, HalfTaperedColor, SDPG_World);
		}
		else
		{
			DrawArc(PDI, BottomEnd, XAxis, YAxis, -90.f, 90.f, ScaledRadius1, DrawCollisionSides, Color, SDPG_World);
			DrawArc(PDI, BottomEnd, XAxis, YAxis, 90.f, 270.f, ScaledRadius1, DrawCollisionSides, HalfTaperedColor, SDPG_World);
		}

		const FVector NegZAxis = -ZAxis;

		const float OffsetZ0 = FMath::IsNearlyZero(Distance) ? 0.f : ((ScaledRadius0 - ScaledRadius1) * ScaledRadius0) / Distance;
		float TopLimit = FMath::IsNearlyZero(ScaledRadius0) ? 90.f : 90.0f - FMath::RadiansToDegrees(FMath::Asin(OffsetZ0 / ScaledRadius0));
		DrawArc(PDI, TopEnd, NegZAxis, YAxis, TopLimit, 360.0f - TopLimit, ScaledRadius0, DrawCollisionSides, Color, SDPG_World);
		DrawArc(PDI, TopEnd, NegZAxis, XAxis, TopLimit, 180.f, ScaledRadius0, DrawCollisionSides, Color, SDPG_World);
		DrawArc(PDI, TopEnd, NegZAxis, XAxis, 180.f, 360.0f - TopLimit, ScaledRadius0, DrawCollisionSides, HalfTaperedColor, SDPG_World);

		const float OffsetZ1 = FMath::IsNearlyZero(Distance) ? 0.f : ((ScaledRadius0 - ScaledRadius1) * ScaledRadius1) / Distance;
		float BottomLimit = 180.0f - TopLimit;
		DrawArc(PDI, BottomEnd, ZAxis, YAxis, BottomLimit, 360.0f - BottomLimit, ScaledRadius1, DrawCollisionSides, Color, SDPG_World);
		DrawArc(PDI, BottomEnd, ZAxis, XAxis, BottomLimit, 180.f, ScaledRadius1, DrawCollisionSides, Color, SDPG_World);
		DrawArc(PDI, BottomEnd, ZAxis, XAxis, 180.f, 360.0f - BottomLimit, ScaledRadius1, DrawCollisionSides, HalfTaperedColor, SDPG_World);

		// Draw connecty lines
		FKTaperedCapsuleElem::DrawTaperedCapsuleSides(PDI, CapsuleOrientation, TopEnd, BottomEnd, ScaledRadius0, ScaledRadius1, Color, bIsOneSided);
	}
}

void FKTaperedCapsuleElem::DrawElemWire(class FPrimitiveDrawInterface* PDI, const FTransform& ElemTM, float Scale, const FColor Color) const
{
	DrawElemWire(PDI, ElemTM, FVector(Scale), Color);
}

void FKTaperedCapsuleElem::DrawElemSolid(class FPrimitiveDrawInterface* PDI, const FTransform& ElemTM, float Scale, const FMaterialRenderProxy* MaterialRenderProxy) const
{
	DrawElemSolid(PDI, ElemTM, FVector(Scale), MaterialRenderProxy);
}

void FKTaperedCapsuleElem::DrawElemWire(class FPrimitiveDrawInterface* PDI, const FTransform& ElemTM, const FVector& Scale3D, const FColor Color) const
{
	const FVector Origin = ElemTM.GetLocation();
	const FVector XAxis = ElemTM.GetScaledAxis(EAxis::X);
	const FVector YAxis = ElemTM.GetScaledAxis(EAxis::Y);
	const FVector ZAxis = ElemTM.GetScaledAxis(EAxis::Z);
	const float ScaledHalfLength = GetScaledCylinderLength(Scale3D) * .5f;
	float ScaledRadius0;
	float ScaledRadius1;
	GetScaledRadii(Scale3D, ScaledRadius0, ScaledRadius1);

	if (IsExtruded())
	{
		const float ScaledHalfWidth = Width * Scale3D.X * .5f;
		const FVector FrontOrigin = Origin + (ScaledHalfWidth * XAxis);
		const FVector BackOrigin = Origin - (ScaledHalfWidth * XAxis);
		DrawWireframeTaperedCapsule(PDI, Color, ElemTM.GetRotation(), FrontOrigin, XAxis, YAxis, ZAxis, ScaledHalfLength, ScaledRadius0, ScaledRadius1, IsOneSidedCollision());
		DrawWireframeTaperedCapsule(PDI, Color, ElemTM.GetRotation(), BackOrigin, XAxis, YAxis, ZAxis, ScaledHalfLength, ScaledRadius0, ScaledRadius1, IsOneSidedCollision());


		// Draw connecty lines
		const FVector TopFrontEnd = FrontOrigin + ScaledHalfLength * ZAxis;
		const FVector TopBackEnd = BackOrigin + ScaledHalfLength * ZAxis;
		PDI->DrawLine(TopFrontEnd + ScaledRadius0 * ZAxis, TopBackEnd + ScaledRadius0 * ZAxis, Color, SDPG_World);
		PDI->DrawLine(TopFrontEnd + ScaledRadius0 * YAxis, TopBackEnd + ScaledRadius0 * YAxis, Color, SDPG_World);
		PDI->DrawLine(TopFrontEnd - ScaledRadius0 * YAxis, TopBackEnd - ScaledRadius0 * YAxis, Color, SDPG_World);

		const FVector BottomFrontEnd = FrontOrigin - ScaledHalfLength * ZAxis;
		const FVector BottomBackEnd = BackOrigin - ScaledHalfLength * ZAxis;
		PDI->DrawLine(BottomFrontEnd - ScaledRadius1 * ZAxis, BottomBackEnd - ScaledRadius1 * ZAxis, Color, SDPG_World);
		PDI->DrawLine(BottomFrontEnd + ScaledRadius1 * YAxis, BottomBackEnd + ScaledRadius1 * YAxis, Color, SDPG_World);
		PDI->DrawLine(BottomFrontEnd - ScaledRadius1 * YAxis, BottomBackEnd - ScaledRadius1 * YAxis, Color, SDPG_World);

	}
	else
	{
		DrawWireframeTaperedCapsule(PDI, Color, ElemTM.GetRotation(), Origin, XAxis, YAxis, ZAxis, ScaledHalfLength, ScaledRadius0, ScaledRadius1, IsOneSidedCollision());
	}
}

struct FScopedTaperedCapsuleBuilder
{
	FScopedTaperedCapsuleBuilder(const FKTaperedCapsuleElem& InTaperedCapsule, const FVector& InScale3D, ERHIFeatureLevel::Type InFeatureLevel)
		: MeshBuilder(InFeatureLevel)
	{
		const bool bIsExtrudedCapsule = InTaperedCapsule.IsExtruded();

		const float ScaledLength = InTaperedCapsule.GetScaledCylinderLength(InScale3D);
		const float ScaledHalfLength = ScaledLength * 0.5f;
		float ScaledRadius0;
		float ScaledRadius1;
		InTaperedCapsule.GetScaledRadii(InScale3D, ScaledRadius0, ScaledRadius1);
		const float ScaledHalfWidth = bIsExtrudedCapsule ? InTaperedCapsule.Width * InScale3D.X * .5f : 0.f;;

		// Deal with degenerates
		float SphereOffset0 = ScaledHalfLength;
		float SphereOffset1 = -ScaledHalfLength;
		if (FMath::IsNearlyZero(ScaledLength) || ScaledLength <= FMath::Abs(ScaledRadius0 - ScaledRadius1))
		{
			// Degenerate or one end encompasses the other - we need to draw a sphere, so map one end to the other
			if (ScaledRadius0 > ScaledRadius1)
			{
				SphereOffset1 = SphereOffset0;
				ScaledRadius1 = ScaledRadius0;
			}
			else
			{
				SphereOffset0 = SphereOffset1;
				ScaledRadius0 = ScaledRadius1;
			}
		}

		const int32 NumSides = DrawCollisionSides + (bIsExtrudedCapsule ? 2 : 0);
		const int32 NumRings = (DrawCollisionSides / 2) + 1;

		// The first/last arc are on top of each other.
		const int32 NumVerts = (NumSides + 1) * (NumRings + 1);
		Verts.SetNumUninitialized(NumVerts);

		// Calculate verts for one arc
		ArcVerts.SetNumUninitialized(NumRings + 1);

		// Calc arc split point
		const float OffsetZ = FMath::IsNearlyZero(ScaledLength) ? 0.f : ((ScaledRadius0 - ScaledRadius1) * ScaledRadius0) / ScaledLength;
		const float SplitAngle = FMath::IsNearlyZero(ScaledRadius0) ? 90.f : 90.0f - FMath::RadiansToDegrees(FMath::Asin(OffsetZ / ScaledRadius0));
		const int32 SplitPoint = (int32)((float)NumRings * ((180.0f - SplitAngle) / 180.0f));

		for (int32 RingIdx = 0; RingIdx < NumRings + 1; RingIdx++)
		{
			FDynamicMeshVertex* ArcVert = &ArcVerts[RingIdx];

			// Note- unit sphere, so position always has mag of one. We can just use it for normal!		
			FVector SpherePos;
			SpherePos.X = 0.0f;

			float ZOffset;

			if (RingIdx <= SplitPoint)
			{
				float Angle = ((float)RingIdx / (NumRings - 1)) * UE_PI;
				ZOffset = SphereOffset0;
				SpherePos.Y = ScaledRadius0 * FMath::Sin(Angle);
				SpherePos.Z = ScaledRadius0 * FMath::Cos(Angle);
			}
			else
			{
				float Angle = ((float)(RingIdx - 1) / (NumRings - 1)) * UE_PI;
				ZOffset = SphereOffset1;
				SpherePos.Y = ScaledRadius1 * FMath::Sin(Angle);
				SpherePos.Z = ScaledRadius1 * FMath::Cos(Angle);
			}

			ArcVert->Position = FVector3f(SpherePos + FVector(0, 0, ZOffset));

			ArcVert->SetTangents(
				FVector3f(1, 0, 0),
				FVector3f(0.0f, -SpherePos.Z, SpherePos.Y),
				(FVector3f)SpherePos
			);

			ArcVert->TextureCoordinate[0].X = 0.0f;
			ArcVert->TextureCoordinate[0].Y = ((float)RingIdx / NumRings);
		}

		// Then rotate this arc NumSides+1 times.
		for (int32 SideIdx = 0; SideIdx < DrawCollisionSides + 1; SideIdx++)
		{
			const FRotator3f ArcRotator(0, 360.f * ((float)SideIdx / DrawCollisionSides), 0);
			const FRotationMatrix44f ArcRot(ArcRotator);
			const float XTexCoord = ((float)SideIdx / DrawCollisionSides);

			const FVector3f XOffset(SideIdx <= DrawCollisionSides / 2 ? -ScaledHalfWidth : ScaledHalfWidth, 0, 0);
			const int32 SideIdxToVIxOffset = (bIsExtrudedCapsule && SideIdx > DrawCollisionSides / 2) ? 1 : 0;
			for (int32 VertIdx = 0; VertIdx < NumRings + 1; VertIdx++)
			{
				const int32 VIx = (NumRings + 1) * (SideIdx + SideIdxToVIxOffset) + VertIdx;

				Verts[VIx].Position = ArcRot.TransformPosition(ArcVerts[VertIdx].Position) + XOffset;

				Verts[VIx].SetTangents(
					ArcRot.TransformVector(ArcVerts[VertIdx].TangentX.ToFVector3f()),
					ArcRot.TransformVector(ArcVerts[VertIdx].GetTangentY()),
					ArcRot.TransformVector(ArcVerts[VertIdx].TangentZ.ToFVector3f())
				);

				Verts[VIx].TextureCoordinate[0].X = XTexCoord;
				Verts[VIx].TextureCoordinate[0].Y = ArcVerts[VertIdx].TextureCoordinate[0].Y;
			}

			if (bIsExtrudedCapsule)
			{

				if (SideIdx == DrawCollisionSides / 2)
				{
					// Duplicate vertex on second half
					for (int32 VertIdx = 0; VertIdx < NumRings + 1; VertIdx++)
					{
						const int32 VIx = (NumRings + 1) * (SideIdx + 1) + VertIdx;

						Verts[VIx].Position = ArcRot.TransformPosition(ArcVerts[VertIdx].Position) + FVector3f(ScaledHalfWidth, 0.f, 0.f);

						Verts[VIx].SetTangents(
							ArcRot.TransformVector(ArcVerts[VertIdx].TangentX.ToFVector3f()),
							ArcRot.TransformVector(ArcVerts[VertIdx].GetTangentY()),
							ArcRot.TransformVector(ArcVerts[VertIdx].TangentZ.ToFVector3f())
						);

						Verts[VIx].TextureCoordinate[0].X = XTexCoord;
						Verts[VIx].TextureCoordinate[0].Y = ArcVerts[VertIdx].TextureCoordinate[0].Y;
					}
				}
				else if (SideIdx == DrawCollisionSides)
				{
					// Duplicate vertex on second half
					for (int32 VertIdx = 0; VertIdx < NumRings + 1; VertIdx++)
					{
						const int32 VIx = (NumRings + 1) * (SideIdx + 2) + VertIdx;

						Verts[VIx].Position = ArcRot.TransformPosition(ArcVerts[VertIdx].Position) - FVector3f(ScaledHalfWidth, 0.f, 0.f);

						Verts[VIx].SetTangents(
							ArcRot.TransformVector(ArcVerts[VertIdx].TangentX.ToFVector3f()),
							ArcRot.TransformVector(ArcVerts[VertIdx].GetTangentY()),
							ArcRot.TransformVector(ArcVerts[VertIdx].TangentZ.ToFVector3f())
						);

						Verts[VIx].TextureCoordinate[0].X = XTexCoord;
						Verts[VIx].TextureCoordinate[0].Y = ArcVerts[VertIdx].TextureCoordinate[0].Y;
					}
				}
			}
		}


		{
			// Add all of the vertices to the mesh.
			for (int32 VertIdx = 0; VertIdx < NumVerts; VertIdx++)
			{
				MeshBuilder.AddVertex(Verts[VertIdx]);
			}

			// Add all of the triangles to the mesh.
			for (int32 SideIdx = 0; SideIdx < NumSides; SideIdx++)
			{
				const int32 a0start = (SideIdx + 0) * (NumRings + 1);
				const int32 a1start = (SideIdx + 1) * (NumRings + 1);

				for (int32 RingIdx = 0; RingIdx < NumRings; RingIdx++)
				{
					MeshBuilder.AddTriangle(a0start + RingIdx + 0, a1start + RingIdx + 0, a0start + RingIdx + 1);
					MeshBuilder.AddTriangle(a1start + RingIdx + 0, a1start + RingIdx + 1, a0start + RingIdx + 1);
				}
			}
		}
	}

	~FScopedTaperedCapsuleBuilder()
	{
	}

	FDynamicMeshBuilder MeshBuilder;
	TArray<FDynamicMeshVertex> Verts;
	TArray<FDynamicMeshVertex> ArcVerts;
};

void FKTaperedCapsuleElem::GetElemSolid(const FTransform& ElemTM, const FVector& Scale3D, const FMaterialRenderProxy* MaterialRenderProxy, int32 ViewIndex, FMeshElementCollector& Collector) const
{
	FScopedTaperedCapsuleBuilder TaperedCapsuleBuilder(*this, Scale3D, Collector.GetFeatureLevel());
	TaperedCapsuleBuilder.MeshBuilder.GetMesh(ElemTM.ToMatrixWithScale(), MaterialRenderProxy, SDPG_World, false, false, ViewIndex, Collector);
}

void FKTaperedCapsuleElem::DrawElemSolid(FPrimitiveDrawInterface* PDI, const FTransform& ElemTM, const FVector& Scale3D, const FMaterialRenderProxy* MaterialRenderProxy) const
{
	FScopedTaperedCapsuleBuilder TaperedCapsuleBuilder(*this, Scale3D, PDI->View->GetFeatureLevel());
	TaperedCapsuleBuilder.MeshBuilder.Draw(PDI, ElemTM.ToMatrixWithScale(), MaterialRenderProxy, SDPG_World);
}


/////////////////////////////////////////////////////////////////////////////////////
// FKConvexElem
/////////////////////////////////////////////////////////////////////////////////////

void FKConvexElem::DrawElemWire(FPrimitiveDrawInterface* PDI, const FTransform& ElemTM, const float Scale, const FColor Color) const
{
	const int32 NumIndices = IndexData.Num();
	if(NumIndices > 0 && ensure(NumIndices % 3 == 0))
	{
		// NOTE: With chaos, instead of using a mesh with transformed verts, we use the 
		// VertexData directly, so we don't need to remove the body->elem transform like
		// we did with physx.
		const int32 NumVerts = VertexData.Num();
		TArray<FVector> TransformedVerts;
		TransformedVerts.Reserve(NumVerts);
		for(const FVector& Vert : VertexData)
		{
			TransformedVerts.Add(ElemTM.TransformPosition(Vert));
		}

		for(int32 Base = 0; Base < NumIndices; Base += 3)
		{
			if (IndexData[Base] >= NumVerts || IndexData[Base + 1] >= NumVerts || IndexData[Base + 2] >= NumVerts)
			{
				continue;
			}

			PDI->DrawLine(TransformedVerts[IndexData[Base]], TransformedVerts[IndexData[Base + 1]], Color, SDPG_World);
			PDI->DrawLine(TransformedVerts[IndexData[Base + 1]], TransformedVerts[IndexData[Base + 2]], Color, SDPG_World);
			PDI->DrawLine(TransformedVerts[IndexData[Base + 2]], TransformedVerts[IndexData[Base]], Color, SDPG_World);
		}
	}
	else
	{
		UE_LOGF(LogPhysics, Log, "FKConvexElem::DrawElemWire : No ConvexMesh, so unable to draw.");
	}
}

void FKConvexElem::DrawElemSolid(FPrimitiveDrawInterface* PDI, const FTransform& ElemTM, const float Scale, const FMaterialRenderProxy* MaterialRenderProxy) const
{
	const int32 NumIndices = IndexData.Num();
	if (NumIndices > 0 && ensure(NumIndices % 3 == 0))
	{
		FDynamicMeshBuilder MeshBuilder(PDI->View->GetFeatureLevel());

		const FVector2D DummyUV(0);
		const FVector3f DummyTangentX(1, 0, 0);
		const FVector3f DummyTangentY(0, 1, 0);
		const FVector3f DummyTangentZ(0, 0, 1);

		// NOTE: With chaos, instead of using a mesh with transformed verts, we use the 
		// VertexData directly, so we don't need to remove the body->elem transform like
		// we did with physx.
		const int32 NumVerts = VertexData.Num();
		for (const FVector& Vert : VertexData)
		{
			MeshBuilder.AddVertex((FVector3f)ElemTM.TransformPosition(Vert), FVector2f(DummyUV), DummyTangentX, DummyTangentY, DummyTangentZ, FColor::White);
		}

		for (int32 Base = 0; Base < NumIndices; Base += 3)
		{
			if (IndexData[Base] >= NumVerts || IndexData[Base + 1] >= NumVerts || IndexData[Base + 2] >= NumVerts)
			{
				continue;
			}
			MeshBuilder.AddTriangle(IndexData[Base], IndexData[Base + 1], IndexData[Base + 2]);
		}
		MeshBuilder.Draw(PDI, FMatrix::Identity, MaterialRenderProxy, SDPG_World);
	}
	else
	{
		UE_LOGF(LogPhysics, Log, "FKConvexElem::DrawElemSolid : No ConvexMesh, so unable to draw.");
	}
}

void FKConvexElem::AddCachedSolidConvexGeom(TArray<FDynamicMeshVertex>& VertexBuffer, TArray<uint32>& IndexBuffer, const FColor VertexColor) const
{
	const int32 NumIndices = IndexData.Num();
	if(NumIndices > 0 && ensure(NumIndices % 3 == 0))
	{
		const int32 NumTriangles = NumIndices / 3;

		for(int32 TriIndex = 0; TriIndex < NumTriangles; ++TriIndex)
		{
			const int32 Base = TriIndex * 3;

			// Note: we are swapping the winding order of the triangles here from CW to CCW winding (Left handed coordinates)
			const int32 TriVertexIndex[3] = { IndexData[Base], IndexData[Base + 2], IndexData[Base + 1] };

			const FVector TangentX = (VertexData[TriVertexIndex[1]] - VertexData[TriVertexIndex[0]]).GetSafeNormal();
			// Note: FPlane assumes CW winding in left handed coordinates and we need CCW (That explains the sign here)
			const FVector TangentZ = -FPlane(VertexData[TriVertexIndex[0]], VertexData[TriVertexIndex[1]], VertexData[TriVertexIndex[2]]).GetSafeNormal();
			const FVector TangentY = FVector::CrossProduct(TangentZ, TangentX).GetSafeNormal();

			for(int32 TriVertCount = 0; TriVertCount < 3; ++TriVertCount)
			{
				const int32 Index = TriVertexIndex[TriVertCount];
				FDynamicMeshVertex Vert;
				Vert.Position = (FVector3f)Transform.TransformPosition(VertexData[Index]);
				Vert.Color = VertexColor;
				Vert.SetTangents((FVector3f)TangentX, (FVector3f)TangentY, (FVector3f)TangentZ);
				VertexBuffer.Add(Vert);
				IndexBuffer.Add(VertexBuffer.Num() - 1); // Output indices
			}
		}
	}
	else
	{
		UE_LOGF(LogPhysics, Log, "FKConvexElem::AddCachedSolidConvexGeom : No ConvexMesh, so unable to draw.");
	}
}

/////////////////////////////////////////////////////////////////////////////////////
// FKLevelSetElem
/////////////////////////////////////////////////////////////////////////////////////

void FKLevelSetElem::DrawElemWire(class FPrimitiveDrawInterface* PDI, const FTransform& ElemTM, float Scale, const FColor Color) const
{
	TArray<FBox> Boxes;

	if (GridResolution().GetMax() < 30)	// Don't render individual cells if the resolution is too high
	{
		// Cells with negative Phi values
		const double Threshold = UE_KINDA_SMALL_NUMBER;		// allow slightly greater than zero for visualization purposes
		GetInteriorGridCells(Boxes, UE_KINDA_SMALL_NUMBER);
	}
	else
	{
		// Just render the bounding box of the grid itself
		Boxes.Add(UntransformedAABB());
	}

	PDI->AddReserveLines(SDPG_World, Boxes.Num() * 12);
	for (const FBox& Box : Boxes)
	{
		constexpr float Thickness = 0.f;
		DrawWireBox(PDI, ElemTM.ToMatrixWithScale(), Box, Color, SDPG_World, Thickness);
	}
}

void FKLevelSetElem::DrawElemSolid(class FPrimitiveDrawInterface* PDI, const FTransform& ElemTM, float Scale, const FMaterialRenderProxy* MaterialRenderProxy) const
{
	TArray<FVector3f> Vertices;
	TArray<FIntVector> Tris;
	GetZeroIsosurfaceGridCellFaces(Vertices, Tris);

	FDynamicMeshBuilder MeshBuilder(PDI->View->GetFeatureLevel());
	for (const FVector3f& V : Vertices)
	{
		MeshBuilder.AddVertex(FDynamicMeshVertex(V));
	}
	for (const FIntVector& T : Tris)
	{
		MeshBuilder.AddTriangle(T[0], T[1], T[2]);
	}

	MeshBuilder.Draw(PDI, ElemTM.ToMatrixWithScale(), MaterialRenderProxy, SDPG_World, 0.f);
}

void FKLevelSetElem::GetElemSolid(const FTransform& ElemTM, const FVector& Scale3D, const FMaterialRenderProxy* MaterialRenderProxy, int32 ViewIndex, class FMeshElementCollector& Collector) const
{
	TArray<FVector3f> Vertices;
	TArray<FIntVector> Tris;
	GetZeroIsosurfaceGridCellFaces(Vertices, Tris);

	FDynamicMeshBuilder MeshBuilder(Collector.GetFeatureLevel());
	for (const FVector3f& V : Vertices)
	{
		MeshBuilder.AddVertex(FDynamicMeshVertex(V));
	}
	for (const FIntVector& T : Tris)
	{
		MeshBuilder.AddTriangle(T[0], T[1], T[2]);
	}

	MeshBuilder.GetMesh(ElemTM.ToMatrixWithScale(), MaterialRenderProxy, SDPG_World, false, false, ViewIndex, Collector);
}

/////////////////////////////////////////////////////////////////////////////////////
// FKSkinnedLevelSetElem
/////////////////////////////////////////////////////////////////////////////////////

static void DrawSkinnedLevelSetLattice(class FPrimitiveDrawInterface* PDI, const FMatrix& LocalToWorld, const FColor Color, const Chaos::TWeightedLatticeImplicitObject<Chaos::FLevelSet>* WeightedLevelSet)
{
	const Chaos::TUniformGrid<double, 3>& LatticeGrid = WeightedLevelSet->GetGrid();
	const Chaos::TArrayND<Chaos::FVec3, 3>& DeformedPoints = WeightedLevelSet->GetDeformedPoints();
	const Chaos::TArrayND<bool, 3>& EmptyCells = WeightedLevelSet->GetEmptyCells();
	const Chaos::TArrayND<Chaos::FWeightedLatticeInfluenceData, 3>& BoneData = WeightedLevelSet->GetBoneData();
	const int32 NumUsedBones = WeightedLevelSet->GetUsedBones().Num();
	const int32 SkinnedLatticeBoneWeightLocal = SkinnedLatticeBoneWeight < NumUsedBones ? SkinnedLatticeBoneWeight : -1;

	Chaos::TArrayND<float,3> SkinnedLatticeBoneWeights;
	if (SkinnedLatticeBoneWeightLocal >= 0)
	{
		static float PointSize = 5.f;
		SkinnedLatticeBoneWeights.SetCounts(LatticeGrid, true);

		const Chaos::TVec3<int32> NodeCounts = LatticeGrid.NodeCounts();
		for (int32 I = 0; I < NodeCounts.X; ++I)
		{
			for (int32 J = 0; J < NodeCounts.Y; ++J)
			{
				for (int32 K = 0; K < NodeCounts.Z; ++K)
				{
					SkinnedLatticeBoneWeights(I, J, K) = 0.f;
					const Chaos::FWeightedLatticeInfluenceData& BoneDatum = BoneData(I, J, K);
					for (int32 InfIdx = 0; InfIdx < BoneDatum.NumInfluences; ++InfIdx)
					{
						if (BoneDatum.BoneIndices[InfIdx] == SkinnedLatticeBoneWeightLocal)
						{
							const float Weight = BoneDatum.BoneWeights[InfIdx];
							SkinnedLatticeBoneWeights(I, J, K) = Weight;
							PDI->DrawPoint(LocalToWorld.TransformPosition(FVector(DeformedPoints(I, J, K))), FLinearColor(Weight, Weight, Weight), PointSize, SDPG_World);
							break;
						}
					}
				}
			}
		}
	}

	auto ShouldDrawCell = [&SkinnedLatticeBoneWeights, &EmptyCells](int32 I, int32 J, int32 K)
	{
		if (EmptyCells(I, J, K))
		{
			return false;
		}
		if (SkinnedLatticeBoneWeights.Num() == 0)
		{
			return true;
		}
		if (SkinnedLatticeBoneWeights(I, J, K) == 0.f ||
			SkinnedLatticeBoneWeights(I, J, K + 1) == 0.f ||
			SkinnedLatticeBoneWeights(I, J + 1, K) == 0.f ||
			SkinnedLatticeBoneWeights(I, J + 1, K + 1) == 0.f ||
			SkinnedLatticeBoneWeights(I + 1, J, K) == 0.f ||
			SkinnedLatticeBoneWeights(I + 1, J, K + 1) == 0.f ||
			SkinnedLatticeBoneWeights(I + 1, J + 1, K) == 0.f ||
			SkinnedLatticeBoneWeights(I + 1, J + 1, K + 1) == 0.f)
		{
			return false;
		}
		return true;
	};

	const Chaos::TVec3<int32> CellCounts = LatticeGrid.Counts();
	for (int32 I = 0; I < CellCounts.X; ++I)
	{
		for (int32 J = 0; J < CellCounts.Y; ++J)
		{
			for (int32 K = 0; K < CellCounts.Z; ++K)
			{
				if (ShouldDrawCell(I, J, K))
				{
					const FVector P000 = LocalToWorld.TransformPosition(FVector(DeformedPoints(I, J, K)));
					const FVector P001 = LocalToWorld.TransformPosition(FVector(DeformedPoints(I, J, K + 1)));
					const FVector P010 = LocalToWorld.TransformPosition(FVector(DeformedPoints(I, J + 1, K)));
					const FVector P011 = LocalToWorld.TransformPosition(FVector(DeformedPoints(I, J + 1, K + 1)));
					const FVector P100 = LocalToWorld.TransformPosition(FVector(DeformedPoints(I + 1, J, K)));
					const FVector P101 = LocalToWorld.TransformPosition(FVector(DeformedPoints(I + 1, J, K + 1)));
					const FVector P110 = LocalToWorld.TransformPosition(FVector(DeformedPoints(I + 1, J + 1, K)));
					const FVector P111 = LocalToWorld.TransformPosition(FVector(DeformedPoints(I + 1, J + 1, K + 1)));

					PDI->AddReserveLines(SDPG_World, 12);
					PDI->DrawLine(P000, P001, Color, SDPG_World);
					PDI->DrawLine(P000, P010, Color, SDPG_World);
					PDI->DrawLine(P000, P100, Color, SDPG_World);
					PDI->DrawLine(P001, P011, Color, SDPG_World);
					PDI->DrawLine(P001, P101, Color, SDPG_World);
					PDI->DrawLine(P010, P011, Color, SDPG_World);
					PDI->DrawLine(P010, P110, Color, SDPG_World);
					PDI->DrawLine(P011, P111, Color, SDPG_World);
					PDI->DrawLine(P100, P101, Color, SDPG_World);
					PDI->DrawLine(P100, P110, Color, SDPG_World);
					PDI->DrawLine(P101, P111, Color, SDPG_World);
					PDI->DrawLine(P110, P111, Color, SDPG_World);
				}
			}
		}
	}

}

void FKSkinnedLevelSetElem::DrawElemWire(class FPrimitiveDrawInterface* PDI, const FTransform& ElemTM, float Scale, const FColor Color) const
{
	if (WeightedLatticeLevelSet.IsValid())
	{
		constexpr float HSVHueShift = 240.f;
		constexpr float HSVSaturationMult = 0.5f;
		constexpr float HSVValueMult = 0.8f;
		constexpr float HSVAlphaMult = 0.5f;
		FLinearColor ShiftedColorHSV = Color.ReinterpretAsLinear().LinearRGBToHSV();
		ShiftedColorHSV.R += HSVHueShift;
		if (ShiftedColorHSV.R >= 360.f)
		{
			ShiftedColorHSV.R -= 360.f;
		}
		ShiftedColorHSV.G *= HSVSaturationMult;
		ShiftedColorHSV.B *= HSVValueMult;
		ShiftedColorHSV.A *= HSVAlphaMult;

		DrawSkinnedLevelSetLattice(PDI, ElemTM.ToMatrixWithScale(), ShiftedColorHSV.HSVToLinearRGB().ToFColor(true), WeightedLatticeLevelSet.GetReference());
	}
}

void FKSkinnedLevelSetElem::DrawElemSolid(class FPrimitiveDrawInterface* PDI, const FTransform& ElemTM, float Scale, const FMaterialRenderProxy* MaterialRenderProxy) const
{
	if (WeightedLatticeLevelSet.IsValid())
	{
		TArray<FVector3f> Vertices;
		TArray<FIntVector> Tris;
		const Chaos::FLevelSet* const LevelSet = WeightedLatticeLevelSet->GetEmbeddedObject();
		LevelSet->GetZeroIsosurfaceGridCellFaces(Vertices, Tris);

		FDynamicMeshBuilder MeshBuilder(PDI->View->GetFeatureLevel());
		for (const FVector3f& V : Vertices)
		{
			MeshBuilder.AddVertex(FDynamicMeshVertex(FVector3f(WeightedLatticeLevelSet->GetDeformedPoint(Chaos::FVec3(V)))));
		}
		for (const FIntVector& T : Tris)
		{
			MeshBuilder.AddTriangle(T[0], T[1], T[2]);
		}

		MeshBuilder.Draw(PDI, ElemTM.ToMatrixWithScale(), MaterialRenderProxy, SDPG_World, 0.f);
	}
}

void FKSkinnedLevelSetElem::GetElemSolid(const FTransform& ElemTM, const FVector& Scale3D, const FMaterialRenderProxy* MaterialRenderProxy, int32 ViewIndex, class FMeshElementCollector& Collector) const
{
	if (WeightedLatticeLevelSet.IsValid())
	{
		TArray<FVector3f> Vertices;
		TArray<FIntVector> Tris;
		const Chaos::FLevelSet* const LevelSet = WeightedLatticeLevelSet->GetEmbeddedObject();
		LevelSet->GetZeroIsosurfaceGridCellFaces(Vertices, Tris);

		FDynamicMeshBuilder MeshBuilder(Collector.GetPDI(ViewIndex)->View->GetFeatureLevel());
		for (const FVector3f& V : Vertices)
		{
			MeshBuilder.AddVertex(FDynamicMeshVertex(FVector3f(WeightedLatticeLevelSet->GetDeformedPoint(Chaos::FVec3(V)))));
		}
		for (const FIntVector& T : Tris)
		{
			MeshBuilder.AddTriangle(T[0], T[1], T[2]);
		}

		MeshBuilder.GetMesh(ElemTM.ToMatrixWithScale(), MaterialRenderProxy, SDPG_World, false, false, ViewIndex, Collector);
	}
}

/////////////////////////////////////////////////////////////////////////////////////
// FKMLLevelSetElem
/////////////////////////////////////////////////////////////////////////////////////

void FKMLLevelSetElem::DrawElemWire(class FPrimitiveDrawInterface* PDI, const FTransform& ElemTM, float Scale, const FColor Color) const
{
	TArray<FBox> Boxes;

	if (GridResolution().GetMax() < 30)	// Don't render individual cells if the resolution is too high
	{
		// Cells with negative Phi values
		const double Threshold = UE_KINDA_SMALL_NUMBER;		// allow slightly greater than zero for visualization purposes
		GetInteriorGridCells(Boxes, UE_KINDA_SMALL_NUMBER);
	}
	else
	{
		// Just render the bounding box of the grid itself
		Boxes.Add(UntransformedAABB());
	}

	PDI->AddReserveLines(SDPG_World, Boxes.Num() * 12);
	for (const FBox& Box : Boxes)
	{
		constexpr float Thickness = 0.f;
		DrawWireBox(PDI, ElemTM.ToMatrixWithScale(), Box, Color, SDPG_World, Thickness);
	}
}

void FKMLLevelSetElem::DrawElemSolid(class FPrimitiveDrawInterface* PDI, const FTransform& ElemTM, float Scale, const FMaterialRenderProxy* MaterialRenderProxy) const
{
	TArray<FVector3f> Vertices;
	TArray<FIntVector> Tris;
	GetZeroIsosurfaceGridCellFaces(Vertices, Tris);

	FDynamicMeshBuilder MeshBuilder(PDI->View->GetFeatureLevel());
	for (const FVector3f& V : Vertices)
	{
		MeshBuilder.AddVertex(FDynamicMeshVertex(V));
	}
	for (const FIntVector& T : Tris)
	{
		MeshBuilder.AddTriangle(T[0], T[1], T[2]);
	}

	MeshBuilder.Draw(PDI, ElemTM.ToMatrixWithScale(), MaterialRenderProxy, SDPG_World, 0.f);
}

void FKMLLevelSetElem::GetElemSolid(const FTransform& ElemTM, const FVector& Scale3D, const FMaterialRenderProxy* MaterialRenderProxy, int32 ViewIndex, class FMeshElementCollector& Collector) const
{
	TArray<FVector3f> Vertices;
	TArray<FIntVector> Tris;
	GetZeroIsosurfaceGridCellFaces(Vertices, Tris);

	FDynamicMeshBuilder MeshBuilder(Collector.GetFeatureLevel());
	for (const FVector3f& V : Vertices)
	{
		MeshBuilder.AddVertex(FDynamicMeshVertex(V));
	}
	for (const FIntVector& T : Tris)
	{
		MeshBuilder.AddTriangle(T[0], T[1], T[2]);
	}

	MeshBuilder.GetMesh(ElemTM.ToMatrixWithScale(), MaterialRenderProxy, SDPG_World, false, false, ViewIndex, Collector);
}

/////////////////////////////////////////////////////////////////////////////////////
// FKSkinnedTriangleMesh
/////////////////////////////////////////////////////////////////////////////////////
void FKSkinnedTriangleMeshElem::DrawElemWire(class FPrimitiveDrawInterface* PDI, const FTransform& ElemTM, float Scale, const FColor Color) const
{
	if (SkinnedTriangleMesh.IsValid())
	{
		const TArray<Chaos::TVec3<int32>>& Elements = SkinnedTriangleMesh->GetTriangleMesh().GetElements();
		const TConstArrayView<Chaos::Softs::FSolverVec3>& Positions = SkinnedTriangleMesh->GetLocalPositions();
		PDI->AddReserveLines(SDPG_World, Elements.Num() * 3);
		for (const Chaos::TVec3<int32>& Element : Elements)
		{
			const FVector P0 = ElemTM.TransformPosition(FVector(Positions[Element[0]]));
			const FVector P1 = ElemTM.TransformPosition(FVector(Positions[Element[1]]));
			const FVector P2 = ElemTM.TransformPosition(FVector(Positions[Element[2]]));
			PDI->DrawLine(P0, P1, Color, SDPG_World);
			PDI->DrawLine(P1, P2, Color, SDPG_World);
			PDI->DrawLine(P2, P0, Color, SDPG_World);
		}
	}
}

void FKSkinnedTriangleMeshElem::DrawElemSolid(class FPrimitiveDrawInterface* PDI, const FTransform& ElemTM, const float Scale, const FMaterialRenderProxy* MaterialRenderProxy) const
{
	if (SkinnedTriangleMesh.IsValid())
	{
		const TArray<Chaos::TVec3<int32>>& Elements = SkinnedTriangleMesh->GetTriangleMesh().GetElements();
		const TConstArrayView<Chaos::Softs::FSolverVec3>& Positions = SkinnedTriangleMesh->GetLocalPositions();

		FDynamicMeshBuilder MeshBuilder(PDI->View->GetFeatureLevel());
		for (const Chaos::Softs::FSolverVec3& Position : Positions)
		{
			MeshBuilder.AddVertex(FDynamicMeshVertex(FVector3f(Position)));
		}
		for (const Chaos::TVec3<int32>& Element : Elements)
		{
			MeshBuilder.AddTriangle(Element[0], Element[1], Element[2]);
		}

		MeshBuilder.Draw(PDI, ElemTM.ToMatrixWithScale(), MaterialRenderProxy, SDPG_World, 0.f);
	}
}



void FKSkinnedTriangleMeshElem::GetElemSolid(const FTransform& ElemTM, const FVector& Scale3D, const FMaterialRenderProxy* MaterialRenderProxy, int32 ViewIndex, class FMeshElementCollector& Collector) const
{
	if (SkinnedTriangleMesh.IsValid())
	{
		const TArray<Chaos::TVec3<int32>>& Elements = SkinnedTriangleMesh->GetTriangleMesh().GetElements();
		const TConstArrayView<Chaos::Softs::FSolverVec3>& Positions = SkinnedTriangleMesh->GetLocalPositions();

		FDynamicMeshBuilder MeshBuilder(Collector.GetFeatureLevel());
		for (const Chaos::Softs::FSolverVec3& Position : Positions)
		{
			MeshBuilder.AddVertex(FDynamicMeshVertex(FVector3f(Position)));
		}
		for (const Chaos::TVec3<int32>& Element : Elements)
		{
			MeshBuilder.AddTriangle(Element[0], Element[1], Element[2]);
		}

		MeshBuilder.GetMesh(ElemTM.ToMatrixWithScale(), MaterialRenderProxy, SDPG_World, false, false, ViewIndex, Collector);
	}
}

/////////////////////////////////////////////////////////////////////////////////////
// FKConvexGeomRenderInfo
/////////////////////////////////////////////////////////////////////////////////////

FKConvexGeomRenderInfo::FKConvexGeomRenderInfo()
	: VertexBuffers(nullptr)
	, IndexBuffer(nullptr)
	, CollisionVertexFactory(nullptr)
{

}

bool FKConvexGeomRenderInfo::HasValidGeometry()
{
	return
		(VertexBuffers != NULL) &&
		(VertexBuffers->PositionVertexBuffer.GetNumVertices() > 0) &&
		(IndexBuffer != NULL) &&
		(IndexBuffer->Indices.Num() > 0);
}

/////////////////////////////////////////////////////////////////////////////////////
// FKAggregateGeom
/////////////////////////////////////////////////////////////////////////////////////

void FKAggregateGeom::GetAggGeom(const FTransform& Transform, const FColor Color, const FMaterialRenderProxy* MatInst, bool bPerHullColor, bool bDrawSolid, bool bOutputVelocity, int32 ViewIndex, FMeshElementCollector& Collector) const
{
	const FVector Scale3D = Transform.GetScale3D();
	FTransform ParentTM = Transform;
	ParentTM.RemoveScaling();

	for (int32 i = 0; i < SphereElems.Num(); i++)
	{
		FTransform ElemTM = SphereElems[i].GetTransform();
		ElemTM.ScaleTranslation(Scale3D);
		ElemTM *= ParentTM;

		if (bDrawSolid)
			SphereElems[i].GetElemSolid(ElemTM, Scale3D, MatInst, ViewIndex, Collector);
		else
			SphereElems[i].DrawElemWire(Collector.GetPDI(ViewIndex), ElemTM, Scale3D, Color);
	}

	for (int32 i = 0; i < BoxElems.Num(); i++)
	{
		const FKBoxElem ScaledBox = BoxElems[i].GetFinalScaled(Scale3D, FTransform::Identity);
		FTransform ElemTM = ScaledBox.GetTransform();
		ElemTM *= ParentTM;

		if (bDrawSolid)
			BoxElems[i].GetElemSolid(ElemTM, Scale3D, MatInst, ViewIndex, Collector);
		else
			BoxElems[i].DrawElemWire(Collector.GetPDI(ViewIndex), ElemTM, Scale3D, Color);
	}

	for (int32 i = 0; i < SphylElems.Num(); i++)
	{
		const FKSphylElem ScaledSphyl = SphylElems[i].GetFinalScaled(Scale3D, FTransform::Identity);  
		FTransform ElemTM = ScaledSphyl.GetTransform();
		ElemTM *= ParentTM;

		if (bDrawSolid)
			SphylElems[i].GetElemSolid(ElemTM, Scale3D, MatInst, ViewIndex, Collector);
		else
			SphylElems[i].DrawElemWire(Collector.GetPDI(ViewIndex), ElemTM, Scale3D, Color);
	}

	FRHICommandListBase& RHICmdList = Collector.GetRHICommandList();

	if(ConvexElems.Num() > 0)
	{
		if(bDrawSolid)
		{
			// Cache collision vertex/index buffer
			auto RenderInfo = RenderInfoPtr.load(std::memory_order_relaxed);
			if(!RenderInfo)
			{
				UE::TScopeLock Lock(RenderInfoLock);

				// After obtaining the lock, we might now have render info, in which case skip re-creating the data
				RenderInfo = RenderInfoPtr.load(std::memory_order_acquire);
				if (!RenderInfo)
				{
					RenderInfo = new FKConvexGeomRenderInfo();
					RenderInfo->VertexBuffers = new FStaticMeshVertexBuffers();
					RenderInfo->IndexBuffer = new FDynamicMeshIndexBuffer32();

					TArray<FDynamicMeshVertex> OutVerts;
					for(int32 i=0; i<ConvexElems.Num(); i++)
					{
						// Get vertices/triangles from this hull.
						ConvexElems[i].AddCachedSolidConvexGeom(OutVerts, RenderInfo->IndexBuffer->Indices, FColor::White);
					}

					// Only continue if we actually got some valid geometry
					// Will crash if we try to init buffers with no data
					if(RenderInfo->VertexBuffers
						&& RenderInfo->IndexBuffer
						&& OutVerts.Num() > 0
						&& RenderInfo->IndexBuffer->Indices.Num() > 0)
					{
						RenderInfo->IndexBuffer->InitResource(RHICmdList);

						RenderInfo->CollisionVertexFactory = new FLocalVertexFactory(Collector.GetFeatureLevel(), "FKAggregateGeom");
						RenderInfo->VertexBuffers->InitFromDynamicVertex(RHICmdList, RenderInfo->CollisionVertexFactory, OutVerts);
					}

					RenderInfoPtr.store(RenderInfo, std::memory_order_release);
				}
			}

			// If we have geometry to draw, do so
			if(RenderInfo->HasValidGeometry())
			{
				// Calculate transform
				FTransform LocalToWorld = FTransform(FQuat::Identity, FVector::ZeroVector, Scale3D) * ParentTM;

				// Draw the mesh.
				FMeshBatch& Mesh = Collector.AllocateMesh();
				FMeshBatchElement& BatchElement = Mesh.Elements[0];
				BatchElement.IndexBuffer = RenderInfo->IndexBuffer;
				Mesh.VertexFactory = RenderInfo->CollisionVertexFactory;
				Mesh.MaterialRenderProxy = MatInst;
				FBoxSphereBounds WorldBounds, LocalBounds;
				CalcBoxSphereBounds(WorldBounds, LocalToWorld);
				CalcBoxSphereBounds(LocalBounds, FTransform::Identity);

				FDynamicPrimitiveUniformBuffer& DynamicPrimitiveUniformBuffer = Collector.AllocateOneFrameResource<FDynamicPrimitiveUniformBuffer>();
				DynamicPrimitiveUniformBuffer.Set(Collector.GetRHICommandList(), LocalToWorld.ToMatrixWithScale(), LocalToWorld.ToMatrixWithScale(), WorldBounds, LocalBounds, true, false, bOutputVelocity);
				BatchElement.PrimitiveUniformBufferResource = &DynamicPrimitiveUniformBuffer.UniformBuffer;

			 	// previous l2w not used so treat as static
				BatchElement.FirstIndex = 0;
				BatchElement.NumPrimitives = RenderInfo->IndexBuffer->Indices.Num() / 3;
				BatchElement.MinVertexIndex = 0;
				BatchElement.MaxVertexIndex = RenderInfo->VertexBuffers->PositionVertexBuffer.GetNumVertices() - 1;
				Mesh.ReverseCulling = LocalToWorld.GetDeterminant() < 0.0f ? true : false;
				Mesh.Type = PT_TriangleList;
				Mesh.DepthPriorityGroup = SDPG_World;
				Mesh.bCanApplyViewModeOverrides = false;
				Collector.AddMesh(ViewIndex, Mesh);
			}
		}
		else
		{
			for(int32 i=0; i<ConvexElems.Num(); i++)
			{
				FColor ConvexColor = bPerHullColor ? DebugUtilColor[i%NUM_DEBUG_UTIL_COLORS] : Color;
				FTransform ElemTM = ConvexElems[i].GetTransform();
				ElemTM *= Transform;
				ConvexElems[i].DrawElemWire(Collector.GetPDI(ViewIndex), ElemTM, 1.f, ConvexColor);	//we pass in 1 for scale because the ElemTM already has the scale baked into it
			}
		}
	}

	for (int32 i = 0; i < TaperedCapsuleElems.Num(); i++)
	{
		const FKTaperedCapsuleElem ScaledTaperedCapsule = TaperedCapsuleElems[i].GetFinalScaled(Scale3D, FTransform::Identity);
		FTransform ElemTM = ScaledTaperedCapsule.GetTransform();
		ElemTM *= ParentTM;

		if (bDrawSolid)
		{
			TaperedCapsuleElems[i].GetElemSolid(ElemTM, Scale3D, MatInst, ViewIndex, Collector);
		}
		else
		{
			TaperedCapsuleElems[i].DrawElemWire(Collector.GetPDI(ViewIndex), ElemTM, Scale3D, Color);
		}
	}

	for (int32 i = 0; i < LevelSetElems.Num(); i++)
	{
		FTransform ElemTM = LevelSetElems[i].GetTransform();
		ElemTM *= Transform;

		if (bDrawSolid)
		{
			LevelSetElems[i].GetElemSolid(ElemTM, Scale3D, MatInst, ViewIndex, Collector);
		}
		else
		{
			LevelSetElems[i].DrawElemWire(Collector.GetPDI(ViewIndex), ElemTM, 1.f, Color);
		}
	}

	for (int32 i = 0; i < SkinnedLevelSetElems.Num(); i++)
	{
		if (bDrawSolid)
		{
			SkinnedLevelSetElems[i].GetElemSolid(Transform, Scale3D, MatInst, ViewIndex, Collector);
		}
		else
		{
			SkinnedLevelSetElems[i].DrawElemWire(Collector.GetPDI(ViewIndex), Transform, 1.f, Color);
		}
	}

	for (int32 i = 0; i < MLLevelSetElems.Num(); i++)
	{
		if (bDrawSolid)
		{
			MLLevelSetElems[i].GetElemSolid(Transform, Scale3D, MatInst, ViewIndex, Collector);
		}
		else
		{
			MLLevelSetElems[i].DrawElemWire(Collector.GetPDI(ViewIndex), Transform, 1.f, Color);
		}
	}

	for (int32 i = 0; i < SkinnedTriangleMeshElems.Num(); i++)
	{
		if (bDrawSolid)
		{
			SkinnedTriangleMeshElems[i].GetElemSolid(Transform, Scale3D, MatInst, ViewIndex, Collector);
		}
		else
		{
			SkinnedTriangleMeshElems[i].DrawElemWire(Collector.GetPDI(ViewIndex), Transform, 1.f, Color);
		}
	}
}

/** Release the RenderInfo (if its there) and safely clean up any resources. Can be called from any thread. */
void FKAggregateGeom::FreeRenderInfo()
{
	// See if we have rendering resources to free. We do this in the open (non-transactionally) because of
	// the threaded nature of this operation.
	FKConvexGeomRenderInfo* RenderInfo = AutoRTFM::Open([&]
		{
			return RenderInfoPtr.exchange(nullptr, std::memory_order_acq_rel);
		});

	if (RenderInfo)
	{
		AutoRTFM::OnCommit([this, RenderInfo]
			{
				// Should always have these if RenderInfo exists
				check(nullptr != RenderInfo);
				check(RenderInfo->VertexBuffers);
				check(RenderInfo->IndexBuffer);

				// Fire off a render command to free these resources
				ENQUEUE_RENDER_COMMAND(FKAggregateGeomFreeRenderInfo)(
					[RenderInfoToRelease = RenderInfo](FRHICommandList& RHICmdList)
					{
						RenderInfoToRelease->VertexBuffers->ColorVertexBuffer.ReleaseResource();
						RenderInfoToRelease->VertexBuffers->StaticMeshVertexBuffer.ReleaseResource();
						RenderInfoToRelease->VertexBuffers->PositionVertexBuffer.ReleaseResource();
						RenderInfoToRelease->IndexBuffer->ReleaseResource();

						// May not exist if no geometry was available
						if (RenderInfoToRelease->CollisionVertexFactory != nullptr)
						{
							RenderInfoToRelease->CollisionVertexFactory->ReleaseResource();
						}

						// Free memory.
						delete RenderInfoToRelease->VertexBuffers;
						delete RenderInfoToRelease->IndexBuffer;

						if (RenderInfoToRelease->CollisionVertexFactory != nullptr)
						{
							delete RenderInfoToRelease->CollisionVertexFactory;
						}

						delete RenderInfoToRelease;
					});
			});

		AutoRTFM::OnAbort([this, RenderInfo]
			{
				check(nullptr != RenderInfo);
				FKConvexGeomRenderInfo* OldRenderInfo = RenderInfoPtr.exchange(RenderInfo, std::memory_order_acq_rel);
				ensure(nullptr == OldRenderInfo);
			});
	}
}

/////////////////////////////////////////////////////////////////////////////////////
// UPhysicsAsset
/////////////////////////////////////////////////////////////////////////////////////

FTransform GetSkelBoneTransform(int32 BoneIndex, const TArray<FTransform>& SpaceBases, const FTransform& LocalToWorld)
{
	if(BoneIndex != INDEX_NONE && BoneIndex < SpaceBases.Num())
	{
		return SpaceBases[BoneIndex] * LocalToWorld;
	}
	else
	{
		return FTransform::Identity;
	}
}

void UPhysicsAsset::GetCollisionMesh(int32 ViewIndex, FMeshElementCollector& Collector, const FReferenceSkeleton& RefSkeleton, const TArray<FTransform>& SpaceBases, const FTransform& LocalToWorld, const FVector& Scale3D) const
{
	for( int32 i=0; i<SkeletalBodySetups.Num(); i++)
	{
		int32 BoneIndex = RefSkeleton.FindBoneIndex( SkeletalBodySetups[i]->BoneName );
		
		FColor* BoneColor = (FColor*)( &SkeletalBodySetups[i] );

		FTransform BoneTransform = GetSkelBoneTransform(BoneIndex, SpaceBases, LocalToWorld);
		// SkelBoneTransform should have the appropriate scale baked in from Component and Import Transform.
		// BoneTransform.SetScale3D(Scale3D);
		if (SkeletalBodySetups[i]->bCreatedPhysicsMeshes)
		{
			SkeletalBodySetups[i]->AggGeom.GetAggGeom(BoneTransform, *BoneColor, NULL, false, false, true, ViewIndex, Collector);
		}
	}
}

void UPhysicsAsset::GetUsedMaterials(TArray<UMaterialInterface*>& Materials)
{
	for (int32 i = 0; i < ConstraintSetup.Num(); i++)
	{
		FConstraintInstance& Instance = ConstraintSetup[i]->DefaultInstance;
		Instance.GetUsedMaterials(Materials);
	}
}

void UPhysicsAsset::DrawConstraints(int32 ViewIndex, FMeshElementCollector& Collector, const FReferenceSkeleton& RefSkeleton, const TArray<FTransform>& SpaceBases, const FTransform& LocalToWorld, float Scale) const
{
	for (int32 i = 0; i < ConstraintSetup.Num(); i++)
	{
		FConstraintInstance& Instance = ConstraintSetup[i]->DefaultInstance;

		// Get each constraint frame in world space.
		FTransform Con1Frame = FTransform::Identity;
		int32 Bone1Index = RefSkeleton.FindBoneIndex(Instance.ConstraintBone1);
		if (Bone1Index != INDEX_NONE)
		{
			FTransform Body1TM = GetSkelBoneTransform(Bone1Index, SpaceBases, LocalToWorld);
			Body1TM.RemoveScaling();
			Con1Frame = Instance.GetRefFrame(EConstraintFrame::Frame1) * Body1TM;
		}

		FTransform Con2Frame = FTransform::Identity;
		int32 Bone2Index = RefSkeleton.FindBoneIndex(Instance.ConstraintBone2);
		if (Bone2Index != INDEX_NONE)
		{
			FTransform Body2TM = GetSkelBoneTransform(Bone2Index, SpaceBases, LocalToWorld);
			Body2TM.RemoveScaling();
			Con2Frame = Instance.GetRefFrame(EConstraintFrame::Frame2) * Body2TM;
		}


		Instance.DrawConstraint(ViewIndex, Collector, Scale, 1.f, true, true, Con1Frame, Con2Frame, false, false);
	}
}
static void DrawLinearLimit(FPrimitiveDrawInterface* PDI, const FVector& Origin, const FVector& Axis, const FVector& Orth, float LinearLimitRadius, bool bLinearLimited, float DrawScale)
{
	float ScaledLimitSize = LimitRenderSize * DrawScale;

	if (bLinearLimited)
	{
		FVector Start = Origin - LinearLimitRadius * Axis;
		FVector End = Origin + LinearLimitRadius * Axis;

		PDI->DrawLine(  Start, End, JointLimitColor, SDPG_World );

		// Draw ends indicating limit.
		PDI->DrawLine(  Start - (0.2f * ScaledLimitSize * Orth), Start + (0.2f * ScaledLimitSize * Orth), JointLimitColor, SDPG_World );
		PDI->DrawLine(  End - (0.2f * ScaledLimitSize * Orth), End + (0.2f * ScaledLimitSize * Orth), JointLimitColor, SDPG_World );
	}
	else
	{
		FVector Start = Origin - 1.5f * ScaledLimitSize * Axis;
		FVector End = Origin + 1.5f * ScaledLimitSize * Axis;

		PDI->DrawLine(  Start, End, JointRefColor, SDPG_World );

		// Draw arrow heads.
		PDI->DrawLine(  Start, Start + (0.2f * ScaledLimitSize * Axis) + (0.2f * ScaledLimitSize * Orth), JointLimitColor, SDPG_World );
		PDI->DrawLine(  Start, Start + (0.2f * ScaledLimitSize * Axis) - (0.2f * ScaledLimitSize * Orth), JointLimitColor, SDPG_World );

		PDI->DrawLine(  End, End - (0.2f * ScaledLimitSize * Axis) + (0.2f * ScaledLimitSize * Orth), JointLimitColor, SDPG_World );
		PDI->DrawLine(  End, End - (0.2f * ScaledLimitSize * Axis) - (0.2f * ScaledLimitSize * Orth), JointLimitColor, SDPG_World );
	}
}

//builds radians for limit based on limit type
float HelpBuildAngle(float LimitAngle, EAngularConstraintMotion LimitType)
{
	switch (LimitType)
	{
		case ACM_Free: return UE_PI;
		case ACM_Locked: return 0.f;
		default: return FMath::DegreesToRadians(LimitAngle);
	}
}

FPrimitiveDrawInterface* FConstraintInstance::FPDIOrCollector::GetPDI() const
{
	return PDI ? PDI : Collector->GetPDI(ViewIndex);
}

void FConstraintInstance::FPDIOrCollector::DrawCylinder(const FVector& Start, const FVector& End, const float Thickness, const FMaterialRenderProxy* const MaterialProxy, const ESceneDepthPriorityGroup DepthPriority) const
{
	if (HasCollector())
	{
		GetCylinderMesh(Start, End, Thickness, 4, MaterialProxy, DepthPriority, ViewIndex, *Collector);
	}
	else
	{
		::DrawCylinder(PDI, Start, End, Thickness, 4, MaterialProxy, DepthPriority);
	}
}

void FConstraintInstance::FPDIOrCollector::DrawCone(const FMatrix& ConeTransform, const float AngleWidth, const float AngleHeight, const uint32 NumSides, const FColor& PDIColor, const FMaterialRenderProxy* MaterialRenderProxy, const ESceneDepthPriorityGroup DepthPriority) const
{
	if (HasCollector())
	{
		GetConeMesh(ConeTransform, FMath::RadiansToDegrees(AngleWidth), FMath::RadiansToDegrees(AngleHeight), NumSides, MaterialRenderProxy, DepthPriority, ViewIndex, *Collector);
	}
	else
	{
		::DrawCone(GetPDI(), ConeTransform, AngleWidth, AngleHeight, NumSides, false, PDIColor, MaterialRenderProxy, DepthPriority);
	}
}

void FConstraintInstance::FPDIOrCollector::DrawArrow(const FMatrix& ArrowTransform, const float Length, const float Thickness, const uint32 NumSides, const FColor& PDIColor, const FMaterialRenderProxy* MaterialRenderProxy, const ESceneDepthPriorityGroup DepthPriority) const
{
	const FVector ArrowDirection = ArrowTransform.GetScaledAxis(EAxis::X);
	const FVector ArrowPosition = ArrowTransform.GetOrigin();
	
	// Draw the arrow shaft as a cylinder.	
	DrawCylinder(ArrowPosition, ArrowPosition + Length * ArrowDirection, Thickness, MaterialRenderProxy, DepthPriority);

	// Draw the arrow head as a cone on the end of the shaft.
	FMatrix ArrowHeadTM = ArrowTransform;
	ArrowHeadTM.SetOrigin(ArrowPosition + Length * 1.05 * ArrowDirection);
	const float ConeAngle = UE_PI / 4;
	const FMatrix ConeToWorld = FScaleMatrix(FVector(Length * -0.1f)) * ArrowHeadTM;

	DrawCone(ConeToWorld, ConeAngle, ConeAngle, NumSides, PDIColor, MaterialRenderProxy, DepthPriority);
}

void FConstraintInstance::GetUsedMaterials(TArray<UMaterialInterface*>& Materials)
{
	Materials.AddUnique(GEngine->ConstraintLimitMaterialX);
	Materials.AddUnique(GEngine->ConstraintLimitMaterialXAxis);
	Materials.AddUnique(GEngine->ConstraintLimitMaterialY);
	Materials.AddUnique(GEngine->ConstraintLimitMaterialYAxis);
	Materials.AddUnique(GEngine->ConstraintLimitMaterialZ);
	Materials.AddUnique(GEngine->ConstraintLimitMaterialZAxis);
}

void FConstraintInstance::DrawConstraintImp(const FPDIOrCollector& PDIOrCollector, float Scale, float LimitDrawScale, bool bDrawLimits, bool bDrawSelected, const FTransform& Con1Frame, const FTransform& Con2Frame, bool bDrawAsPoint, bool bDrawViolatedLimits) const
{
	// Do nothing if we're shipping
#if !UE_BUILD_SHIPPING
	const ESceneDepthPriorityGroup Layer = ESceneDepthPriorityGroup::SDPG_World;
	FPrimitiveDrawInterface* PDI = PDIOrCollector.GetPDI();

	check(GEngine->ConstraintLimitMaterialX && GEngine->ConstraintLimitMaterialY && GEngine->ConstraintLimitMaterialZ);

	static UMaterialInterface* LimitMaterialX = GEngine->ConstraintLimitMaterialX;
	static UMaterialInterface* LimitMaterialXAxis = GEngine->ConstraintLimitMaterialXAxis;
	static UMaterialInterface* LimitMaterialY = GEngine->ConstraintLimitMaterialY;
	static UMaterialInterface* LimitMaterialYAxis = GEngine->ConstraintLimitMaterialYAxis;
	static UMaterialInterface* LimitMaterialZ = GEngine->ConstraintLimitMaterialZ;
	static UMaterialInterface* LimitMaterialZAxis = GEngine->ConstraintLimitMaterialZAxis;

	const FVector Con1Pos = Con1Frame.GetTranslation();
	const FVector Con2Pos = Con2Frame.GetTranslation();

	float Length = (bDrawSelected ? SelectedJointRenderSize : UnselectedJointRenderSize) * Scale;
	float Thickness = JointRenderThickness;

	// Special mode for drawing joints just as points..
	if(bDrawAsPoint && !bDrawSelected)
	{
		PDI->DrawPoint( Con1Frame.GetTranslation(), JointUnselectedColor, ConstraintPointSize, ESceneDepthPriorityGroup::SDPG_Foreground );
		PDI->DrawPoint( Con2Frame.GetTranslation(), JointUnselectedColor, ConstraintPointSize, ESceneDepthPriorityGroup::SDPG_Foreground );

		// do nothing else in this mode.
		return;
	}

	if (bDrawLimits)
	{
		bool bTwistViolated = false;
		bool bSwing1Violated = false;
		bool bSwing2Violated = false;
		if (bDrawViolatedLimits)
		{
			Chaos::FReal TwistAngle, Swing1Angle, Swing2Angle;
			const FQuat ParentQ = Con2Frame.GetRotation();
			FQuat ChildQ = Con1Frame.GetRotation();
			ChildQ.EnforceShortestArcWith(ParentQ);
			Chaos::FPBDJointUtilities::GetSwingTwistAngles(ParentQ, ChildQ, TwistAngle, Swing1Angle, Swing2Angle);
			bTwistViolated = GetAngularTwistMotion() == ACM_Limited && FMath::Abs(TwistAngle) > FMath::DegreesToRadians(GetAngularTwistLimit());
			bSwing1Violated = GetAngularSwing1Motion() == ACM_Limited && FMath::Abs(Swing1Angle) > FMath::DegreesToRadians(GetAngularSwing1Limit());
			bSwing2Violated = GetAngularSwing2Motion() == ACM_Limited && FMath::Abs(Swing2Angle) > FMath::DegreesToRadians(GetAngularSwing2Limit());
		}

		//////////////////////////////////////////////////////////////////////////
		// ANGULAR DRAWING

		//Draw limits first as they are transparent and need to be under coordinate axes
		const bool bLockSwing1 = GetAngularSwing1Motion() == ACM_Locked;
		const bool bLockSwing2 = GetAngularSwing2Motion() == ACM_Locked;
		const bool bLockAllSwing = bLockSwing1 && bLockSwing2;

		// If swing is limited (but not locked) - draw the swing limit cone.
		if (!bLockAllSwing)
		{
			if (ProfileInstance.ConeLimit.Swing1Motion == ACM_Free && ProfileInstance.ConeLimit.Swing2Motion == ACM_Free)
			{
				if (PDIOrCollector.HasCollector())
				{
					GetSphereMesh(Con1Pos, FVector(Length*0.9f), DrawConeLimitSides, DrawConeLimitSides, LimitMaterialX->GetRenderProxy(), Layer, false, PDIOrCollector.ViewIndex, *PDIOrCollector.Collector);
				}
				else
				{
					DrawSphere(PDI, Con1Pos, FRotator::ZeroRotator, FVector(Length * 0.9f), DrawConeLimitSides, DrawConeLimitSides, LimitMaterialX->GetRenderProxy(), Layer);
				}
			}
			else
			{
				const FTransform ConeLimitTM = Con2Frame;
				const float Swing1Ang = HelpBuildAngle(GetAngularSwing1Limit(), GetAngularSwing1Motion());
				const float Swing2Ang = HelpBuildAngle(GetAngularSwing2Limit(), GetAngularSwing2Motion());
				const FMatrix ConeToWorld = FScaleMatrix(FVector(Length * 0.9f)) * ConeLimitTM.ToMatrixWithScale();
				PDIOrCollector.DrawCone(ConeToWorld, Swing1Ang, Swing2Ang, DrawConeLimitSides, JointLimitColor, LimitMaterialX->GetRenderProxy(), Layer);
			}
	
			// Draw the swing Dial indicator - shows the current orientation of the child frame relative to the parent frame on the swing axis.	
			FTransform ArrowTM = Con1Frame;
			ArrowTM.SetTranslation(Con2Pos);
			PDIOrCollector.DrawArrow(ArrowTM.ToMatrixWithScale(), Length, Thickness, DrawConeLimitSides, JointLimitColor, LimitMaterialX->GetRenderProxy(), Layer);

			if (bSwing1Violated || bSwing2Violated)
			{
				PDI->DrawPoint(ArrowTM.TransformPosition(FVector(Length, 0, 0)), JointLimitsViolatedColor, ConstraintLimitViolatedPointSize, ESceneDepthPriorityGroup::SDPG_Foreground);
			}
		}

		// Draw the twist limit - A green arc that shows the allowed range of rotation about the parent frame's x axis.
		if (GetAngularTwistMotion() != ACM_Locked)
		{
			const FTransform ConeLimitTM(Con2Frame.GetScaledAxis(EAxis::Y), Con2Frame.GetScaledAxis(EAxis::Z), Con2Frame.GetScaledAxis(EAxis::X), Con2Frame.GetTranslation()); // Draw the fan in the parent frame (Con2Frame).
			const FMatrix ConeToWorld = FScaleMatrix(FVector(Length * 0.9f)) * ConeLimitTM.ToMatrixWithScale();
			const float LimitAngle = HelpBuildAngle(GetAngularTwistLimit(), GetAngularTwistMotion());
			PDIOrCollector.DrawCone(ConeToWorld, LimitAngle, 0, DrawConeLimitSides, JointLimitColor, LimitMaterialY->GetRenderProxy(), Layer);

			// Draw the twist Dial indicator - shows the current orientation of the child frame relative to the parent frame on the twist axis.
			const FVector TwistIndicator = FVector::PointPlaneProject(Con1Frame.GetScaledAxis(EAxis::Y), FVector::ZeroVector, Con2Frame.GetUnitAxis(EAxis::X)).GetSafeNormal(); // project the y axis of the child frame into the parent frame's yz plane
			const FTransform ArrowTM(TwistIndicator, Con2Frame.GetScaledAxis(EAxis::X), TwistIndicator ^ Con2Frame.GetScaledAxis(EAxis::X), Con2Frame.GetTranslation());
			PDIOrCollector.DrawArrow(ArrowTM.ToMatrixWithScale(), Length, Thickness, DrawConeLimitSides, JointLimitColor, LimitMaterialYAxis->GetRenderProxy(), Layer);

			if (bTwistViolated)
			{
				PDI->DrawPoint(ArrowTM.TransformPosition(FVector(Length, 0, 0)), JointLimitsViolatedColor, ConstraintLimitViolatedPointSize, ESceneDepthPriorityGroup::SDPG_Foreground);
			}

		}
	}

	//////////////////////////////////////////////////////////////////////////
	// COORDINATE AXES

	const float FrameTransformRenderSize = Length * 0.1f;

	// Child Transform
	{
		PDIOrCollector.DrawCylinder(Con1Pos, Con1Pos + FrameTransformRenderSize * Con1Frame.GetScaledAxis(EAxis::X), Thickness, LimitMaterialXAxis->GetRenderProxy(), Layer);
		PDIOrCollector.DrawCylinder(Con1Pos, Con1Pos + FrameTransformRenderSize * Con1Frame.GetScaledAxis(EAxis::Y), Thickness, LimitMaterialYAxis->GetRenderProxy(), Layer);
		PDIOrCollector.DrawCylinder(Con1Pos, Con1Pos + FrameTransformRenderSize * Con1Frame.GetScaledAxis(EAxis::Z), Thickness, LimitMaterialZAxis->GetRenderProxy(), Layer);
	}

	// Parent Transform
	{
		const FVector Position = Con2Frame.GetTranslation();
		PDIOrCollector.DrawCylinder(Con2Pos, Con2Pos + FrameTransformRenderSize * Con2Frame.GetScaledAxis(EAxis::X), Thickness, LimitMaterialXAxis->GetRenderProxy(), Layer);
		PDIOrCollector.DrawCylinder(Con2Pos, Con2Pos + FrameTransformRenderSize * Con2Frame.GetScaledAxis(EAxis::Y), Thickness, LimitMaterialYAxis->GetRenderProxy(), Layer);
		PDIOrCollector.DrawCylinder(Con2Pos, Con2Pos + FrameTransformRenderSize * Con2Frame.GetScaledAxis(EAxis::Z), Thickness, LimitMaterialZAxis->GetRenderProxy(), Layer);
	}

	//////////////////////////////////////////////////////////////////////////
	// LINEAR DRAWING

	//TODO: Move this all into a draw function on linear constraint
	bool bLinearXLocked = (GetLinearXMotion() == LCM_Locked) || (GetLinearXMotion() == LCM_Limited && GetLinearLimit() < RB_MinSizeToLockDOF);
	bool bLinearYLocked = (GetLinearYMotion() == LCM_Locked) || (GetLinearYMotion() == LCM_Limited && GetLinearLimit() < RB_MinSizeToLockDOF);
	bool bLinearZLocked = (GetLinearZMotion() == LCM_Locked) || (GetLinearZMotion() == LCM_Limited && GetLinearLimit() < RB_MinSizeToLockDOF);

	if(!bLinearXLocked)
	{
		bool bLinearXLimited = ( GetLinearXMotion() == LCM_Limited && GetLinearLimit() >= RB_MinSizeToLockDOF );
		DrawLinearLimit(PDI, Con2Frame.GetTranslation(), Con2Frame.GetScaledAxis( EAxis::X ), Con2Frame.GetScaledAxis( EAxis::Z ), GetLinearLimit(), bLinearXLimited, LimitDrawScale);
	}

	if(!bLinearYLocked)
	{
		bool bLinearYLimited = ( GetLinearYMotion() == LCM_Limited && GetLinearLimit() >= RB_MinSizeToLockDOF );
		DrawLinearLimit(PDI, Con2Frame.GetTranslation(), Con2Frame.GetScaledAxis( EAxis::Y ), Con2Frame.GetScaledAxis( EAxis::Z ), GetLinearLimit(), bLinearYLimited, LimitDrawScale);
	}

	if(!bLinearZLocked)
	{
		bool bLinearZLimited = ( GetLinearZMotion() == LCM_Limited && GetLinearLimit() >= RB_MinSizeToLockDOF );
		DrawLinearLimit(PDI, Con2Frame.GetTranslation(), Con2Frame.GetScaledAxis( EAxis::Z ), Con2Frame.GetScaledAxis( EAxis::X ), GetLinearLimit(), bLinearZLimited, LimitDrawScale);
	}
#endif
}
