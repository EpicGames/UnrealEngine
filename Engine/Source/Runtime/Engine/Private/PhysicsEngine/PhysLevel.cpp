// Copyright Epic Games, Inc. All Rights Reserved.

#include "Engine/World.h"
#include "Physics/Experimental/PhysInterface_Chaos.h"
#include "PhysicsInitialization.h"
#include "Physics/Experimental/PhysScene_Chaos.h"
#include "PhysicsEngine/PhysicsSettings.h"

#include "ChaosSolversModule.h"

#include "PhysicsEngine/PhysicsSettings.h"
#include "Misc/CoreDelegates.h"

#ifndef APEX_STATICALLY_LINKED
	#define APEX_STATICALLY_LINKED	0
#endif


FPhysCommandHandler * GPhysCommandHandler = NULL;
FDelegateHandle GPreGarbageCollectDelegateHandle;

FPhysicsDelegates::FOnPhysicsAssetChanged FPhysicsDelegates::OnPhysicsAssetChanged;
FPhysicsDelegates::FOnPhysSceneInit FPhysicsDelegates::OnPhysSceneInit;
FPhysicsDelegates::FOnPhysSceneTerm FPhysicsDelegates::OnPhysSceneTerm;
FPhysicsDelegates::FOnPhysDispatchNotifications FPhysicsDelegates::OnPhysDispatchNotifications;

/**
 *  Chaos is external to engine but utilizes IChaosSettingsProvider to take settings
 *  From external callers, this implementation allows Chaos to request settings from
 *  the engine
 */
class FEngineChaosSettingsProvider : public IChaosSettingsProvider
{
public:

	FEngineChaosSettingsProvider()
		: Settings(nullptr)
	{

	}

	virtual float GetMinDeltaVelocityForHitEvents() const override
	{
		return GetSettings()->MinDeltaVelocityForHitEvents;
	}

	virtual bool GetPhysicsPredictionEnabled() const override
	{
		return GetSettings()->PhysicsPrediction.bEnablePhysicsPrediction;
	}

	UE_DEPRECATED(5.5, "GetResimulationErrorThreshold has been renamed, please use GetResimulationErrorPositionThreshold.")
	virtual float GetResimulationErrorThreshold() const override
	{
		return GetResimulationErrorPositionThreshold();
	}

	virtual bool GetResimulationErrorPositionThresholdEnabled() const
	{
		return GetSettings()->PhysicsPrediction.ResimulationSettings.bEnableResimulationErrorPositionThreshold;
	}

	virtual float GetResimulationErrorPositionThreshold() const
	{
		return GetSettings()->PhysicsPrediction.ResimulationSettings.ResimulationErrorPositionThreshold;
	}

	virtual bool GetResimulationErrorRotationThresholdEnabled() const
	{
		return GetSettings()->PhysicsPrediction.ResimulationSettings.bEnableResimulationErrorRotationThreshold;
	}

	virtual float GetResimulationErrorRotationThreshold() const
	{
		return GetSettings()->PhysicsPrediction.ResimulationSettings.ResimulationErrorRotationThreshold;
	}

	virtual bool GetResimulationErrorLinearVelocityThresholdEnabled() const
	{
		return GetSettings()->PhysicsPrediction.ResimulationSettings.bEnableResimulationErrorLinearVelocityThreshold;
	}

	virtual float GetResimulationErrorLinearVelocityThreshold() const
	{
		return GetSettings()->PhysicsPrediction.ResimulationSettings.ResimulationErrorLinearVelocityThreshold;
	}

	virtual bool GetResimulationErrorAngularVelocityThresholdEnabled() const
	{
		return GetSettings()->PhysicsPrediction.ResimulationSettings.bEnableResimulationErrorAngularVelocityThreshold;
	}

	virtual float GetResimulationErrorAngularVelocityThreshold() const
	{
		return GetSettings()->PhysicsPrediction.ResimulationSettings.ResimulationErrorAngularVelocityThreshold;
	}

	virtual float GetPhysicsHistoryTimeLength() const
	{
		return GetSettings()->PhysicsPrediction.MaxSupportedLatencyPrediction;
	}

	virtual int32 GetPhysicsHistoryCount() const override
	{
		return GetSettings()->GetPhysicsHistoryCount();
	}

private:

	const UPhysicsSettings* GetSettings() const
	{
		if(!Settings)
		{
			Settings = UPhysicsSettings::Get();
		}

		check(Settings);

		return Settings;
	}

	const FChaosPhysicsSettings& GetChaosSettings() const
	{
		return GetSettings()->ChaosSettings;
	}

	const mutable UPhysicsSettings* Settings;

};

static FEngineChaosSettingsProvider GEngineChaosSettingsProvider;

//////////////////////////////////////////////////////////////////////////
// UWORLD
//////////////////////////////////////////////////////////////////////////

void UWorld::SetupPhysicsTickFunctions(float DeltaSeconds)
{
	StartPhysicsTickFunction.bCanEverTick = true;
	StartPhysicsTickFunction.Target = this;
	
	EndPhysicsTickFunction.bCanEverTick = true;
	EndPhysicsTickFunction.Target = this;

// Chaos ticks solver for trace collisions
#if (WITH_EDITOR)
	bool bEnablePhysics = (bShouldSimulatePhysics || bEnableTraceCollision);
#else
	bool bEnablePhysics = bShouldSimulatePhysics;
#endif
	
	// see if we need to update tick registration;
	bool bNeedToUpdateTickRegistration = (bEnablePhysics != StartPhysicsTickFunction.IsTickFunctionRegistered())
		|| (bEnablePhysics != EndPhysicsTickFunction.IsTickFunctionRegistered());

	if (bNeedToUpdateTickRegistration && PersistentLevel)
	{
		if (bEnablePhysics && !StartPhysicsTickFunction.IsTickFunctionRegistered())
		{
			StartPhysicsTickFunction.TickGroup = TG_StartPhysics;
			StartPhysicsTickFunction.RegisterTickFunction(PersistentLevel);
		}
		else if (!bEnablePhysics && StartPhysicsTickFunction.IsTickFunctionRegistered())
		{
			StartPhysicsTickFunction.UnRegisterTickFunction();
		}

		if (bEnablePhysics && !EndPhysicsTickFunction.IsTickFunctionRegistered())
		{
			EndPhysicsTickFunction.TickGroup = TG_EndPhysics;
			EndPhysicsTickFunction.RegisterTickFunction(PersistentLevel);
			EndPhysicsTickFunction.AddPrerequisite(this, StartPhysicsTickFunction);
		}
		else if (!bEnablePhysics && EndPhysicsTickFunction.IsTickFunctionRegistered())
		{
			EndPhysicsTickFunction.RemovePrerequisite(this, StartPhysicsTickFunction);
			EndPhysicsTickFunction.UnRegisterTickFunction();
		}
	}

	FPhysScene* PhysScene = GetPhysicsScene();
	if (PhysicsScene == NULL)
	{
		return;
	}

	
	// When ticking the main scene, clean up any physics engine resources (once a frame)
	DeferredPhysResourceCleanup();

	// Update gravity in case it changed
	FVector DefaultGravity( 0.f, 0.f, GetGravityZ() );

	static const auto CVar_MaxPhysicsDeltaTime = IConsoleManager::Get().FindTConsoleVariableDataFloat(TEXT("p.MaxPhysicsDeltaTime"));

	UPhysicsSettings* Settings = UPhysicsSettings::Get();

	float MinPhysicsDeltaTime = Settings->MinPhysicsDeltaTime;
	float MaxPhysicsDeltaTime = Settings->MaxPhysicsDeltaTime;
	float MaxSubstepDeltaTime = Settings->MaxSubstepDeltaTime;

	/* When using physics prediction, allow max delta time at least equal to max supported latency for physics prediction
	* NOTE: These values clamp how much game thread delta time that can accumulate towards ticking the fixed physics steps, 
	* if we clamp this accumulation too much we hinder time dilation from correcting desyncs and we also make the client (and server) more prone to desyncing physics due to dropping physics steps. */
	if (Settings->PhysicsPrediction.bEnablePhysicsPrediction && Settings->bTickPhysicsAsync)
	{
		MinPhysicsDeltaTime = 0.0f;
		MaxPhysicsDeltaTime = MaxPhysicsDeltaTime <= UE_SMALL_NUMBER ? 0.0f : FMath::Max((Settings->PhysicsPrediction.MaxSupportedLatencyPrediction / 1000.0f), MaxPhysicsDeltaTime);
		MaxSubstepDeltaTime = MaxSubstepDeltaTime <= UE_SMALL_NUMBER ? 0.0f : FMath::Max((Settings->PhysicsPrediction.MaxSupportedLatencyPrediction / 1000.0f) / Settings->MaxSubsteps, MaxSubstepDeltaTime);
	}

	PhysScene->SetUpForFrame(&DefaultGravity, DeltaSeconds, MinPhysicsDeltaTime, MaxPhysicsDeltaTime, MaxSubstepDeltaTime, Settings->MaxSubsteps, Settings->bSubstepping);
}

void UWorld::StartPhysicsSim()
{
	FPhysScene* PhysScene = GetPhysicsScene();
	if (PhysScene == NULL)
	{
		return;
	}

	PhysScene->StartFrame();
}

void UWorld::FinishPhysicsSim()
{
	FPhysScene* PhysScene = GetPhysicsScene();
	if (PhysScene == NULL)
	{
		return;
	}

	PhysScene->EndFrame();
}

// the physics tick functions

void FStartPhysicsTickFunction::ExecuteTick(float DeltaTime, enum ELevelTick TickType, ENamedThreads::Type CurrentThread, const FGraphEventRef& MyCompletionGraphEvent)
{
	QUICK_SCOPE_CYCLE_COUNTER(FStartPhysicsTickFunction_ExecuteTick);
	CSV_SCOPED_TIMING_STAT_EXCLUSIVE(Physics);
	check(Target);
	Target->StartPhysicsSim();
}

FString FStartPhysicsTickFunction::DiagnosticMessage()
{
	return TEXT("FStartPhysicsTickFunction");
}

FName FStartPhysicsTickFunction::DiagnosticContext(bool bDetailed)
{
	return FName("StartPhysicsTick");
}

void FEndPhysicsTickFunction::ExecuteTick(float DeltaTime, enum ELevelTick TickType, ENamedThreads::Type CurrentThread, const FGraphEventRef& MyCompletionGraphEvent)
{
	QUICK_SCOPE_CYCLE_COUNTER(FEndPhysicsTickFunction_ExecuteTick);
	CSV_SCOPED_TIMING_STAT_EXCLUSIVE(Physics);

	check(Target);
	FPhysScene* PhysScene = Target->GetPhysicsScene();
	if (PhysScene == NULL)
	{
		return;
	}

	FGraphEventArray PhysicsComplete = PhysScene->GetCompletionEvents();
	if (!PhysScene->IsCompletionEventComplete())
	{
		// don't release the next tick group until the physics has completed and we have run FinishPhysicsSim
		DECLARE_CYCLE_STAT(TEXT("FSimpleDelegateGraphTask.FinishPhysicsSim"),
			STAT_FSimpleDelegateGraphTask_FinishPhysicsSim,
			STATGROUP_TaskGraphTasks);

		MyCompletionGraphEvent->DontCompleteUntil(
			FSimpleDelegateGraphTask::CreateAndDispatchWhenReady(
				FSimpleDelegateGraphTask::FDelegate::CreateUObject(Target, &UWorld::FinishPhysicsSim),
				GET_STATID(STAT_FSimpleDelegateGraphTask_FinishPhysicsSim), &PhysicsComplete, ENamedThreads::GameThread
			)
		);
	}
	else
	{
		// it was already done, so let just do it.
		Target->FinishPhysicsSim();
	}
}

FString FEndPhysicsTickFunction::DiagnosticMessage()
{
	return TEXT("FEndPhysicsTickFunction");
}

FName FEndPhysicsTickFunction::DiagnosticContext(bool bDetailed)
{
	return FName("EndPhysicsTick");
}

//////// GAME-LEVEL RIGID BODY PHYSICS STUFF ///////
void PostEngineInitialize()
{
	FChaosSolversModule* ChaosModule = FChaosSolversModule::GetModule();

	if(ChaosModule)
	{
		// If the solver module is available, pass along our settings provider
		// #BG - Collect all chaos modules settings into one provider?
		ChaosModule->SetSettingsProvider(&GEngineChaosSettingsProvider);
	}
}

FDelegateHandle GPostInitHandle;

bool InitGamePhys()
{
	if (!InitGamePhysCore())
	{
		return false;
	}

	// We need to defer initializing the module as it will attempt to read from the settings provider. If the settings
	// provider is backed by a UObject in any way access to it will fail because we're too early in the init process.
	GPostInitHandle = FCoreDelegates::GetOnPostEngineInit().AddLambda([]()
	{
		PostEngineInitialize();
	});

	
	GPhysCommandHandler = new FPhysCommandHandler();
	GPreGarbageCollectDelegateHandle = FCoreUObjectDelegates::GetPreGarbageCollectDelegate().AddRaw(GPhysCommandHandler, &FPhysCommandHandler::Flush);

	// One-time register delegate with Trim() to run our deferred cleanup upon request
	static FDelegateHandle Clear = FCoreDelegates::GetMemoryTrimDelegate().AddLambda([]()
	{
		DeferredPhysResourceCleanup();
	});
	

	// Message to the log that physics is initialised and which interface we are using.
	UE_LOGF(LogInit, Log, "Physics initialised using underlying interface: %ls", *FPhysicsInterface::GetInterfaceDescription());

	return true;
}

void TermGamePhys()
{
	if (GPostInitHandle.IsValid())
	{
		FCoreDelegates::GetOnPostEngineInit().Remove(GPostInitHandle);
		GPostInitHandle.Reset();
	}

	if (GPhysCommandHandler != NULL)
	{
		GPhysCommandHandler->Flush();	//finish off any remaining commands
		FCoreUObjectDelegates::GetPreGarbageCollectDelegate().Remove(GPreGarbageCollectDelegateHandle);
		delete GPhysCommandHandler;
		GPhysCommandHandler = NULL;
	}

	TermGamePhysCore();
}

/** 
*	Perform any cleanup of physics engine resources. 
*	This is deferred because when closing down the game, you want to make sure you are not destroying a mesh after the physics SDK has been shut down.
*/
void DeferredPhysResourceCleanup()
{
}

