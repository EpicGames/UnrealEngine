// Copyright Epic Games, Inc. All Rights Reserved.

#include "PhysicsEngine/PhysSubstepTasks.h"
#include "PhysicsEngine/PhysicsSettings.h"

struct FSubstepCallbackGuard
{
#if !UE_BUILD_SHIPPING
	FSubstepCallbackGuard(FPhysSubstepTask& InSubstepTask) : SubstepTask(InSubstepTask)
	{
		++SubstepTask.SubstepCallbackGuard;
	}

	~FSubstepCallbackGuard()
	{
		--SubstepTask.SubstepCallbackGuard;
	}

	FPhysSubstepTask& SubstepTask;
#else
	FSubstepCallbackGuard(FPhysSubstepTask&)
	{
	}
#endif
};

void FPhysSubstepTask::SwapBuffers()
{
	External = !External;
}

void FPhysSubstepTask::RemoveBodyInstance_AssumesLocked(FBodyInstance* BodyInstance)
{
	PhysTargetBuffers[External].Remove(BodyInstance);
	PhysTargetBuffers[!External].Remove(BodyInstance);
}

void FPhysSubstepTask::SetKinematicTarget_AssumesLocked(FBodyInstance* Body, const FTransform& TM)
{
	TM.DiagnosticCheck_IsValid();

	//We only interpolate kinematic actors that need it
	if (!Body->IsNonKinematic() && Body->ShouldInterpolateWhenSubStepping())
	{
		FKinematicTarget_AssumesLocked KinmaticTarget(Body, TM);
		FPhysTarget & TargetState = PhysTargetBuffers[External].FindOrAdd(Body);
		TargetState.bKinematicTarget = true;
		TargetState.KinematicTarget = KinmaticTarget;
	}
}

bool FPhysSubstepTask::GetKinematicTarget_AssumesLocked(const FBodyInstance* Body, FTransform& OutTM) const
{
	if (const FPhysTarget* TargetState = PhysTargetBuffers[External].Find(Body))
	{
		if (TargetState->bKinematicTarget)
		{
			OutTM = TargetState->KinematicTarget.TargetTM;
			return true;
		}
	}

	return false;
}

void FPhysSubstepTask::AddCustomPhysics_AssumesLocked(FBodyInstance* Body, const FCalculateCustomPhysics& CalculateCustomPhysics)
{
	//Limit custom physics to non kinematic actors
	if (Body->IsNonKinematic())
	{
		FCustomTarget CustomTarget(CalculateCustomPhysics);

		FPhysTarget & TargetState = PhysTargetBuffers[External].FindOrAdd(Body);
		TargetState.CustomPhysics.Add(CustomTarget);
	}
}

#if !UE_BUILD_SHIPPING
#define SUBSTEPPING_WARNING() ensureMsgf(SubstepCallbackGuard == 0, TEXT("Applying a sub-stepped force from a substep callback. This usually indicates an error. Make sure you're only using physx data, and that you are adding non-substepped forces"));
#else
#define SUBSTEPPING_WARNING()
#endif

void FPhysSubstepTask::AddForce_AssumesLocked(FBodyInstance* Body, const FVector& Force, bool bAccelChange)
{
	//We should only apply forces on non kinematic actors
	if (Body->IsNonKinematic())
	{
		SUBSTEPPING_WARNING()
		FForceTarget ForceTarget;
		ForceTarget.bPosition = false;
		ForceTarget.Force = Force;
		ForceTarget.bAccelChange = bAccelChange;

		FPhysTarget & TargetState = PhysTargetBuffers[External].FindOrAdd(Body);
		TargetState.Forces.Add(ForceTarget);
	}
}

void FPhysSubstepTask::AddForceAtPosition_AssumesLocked(FBodyInstance* Body, const FVector& Force, const FVector& Position, bool bIsLocalForce)
{
	if (Body->IsNonKinematic())
	{
		SUBSTEPPING_WARNING()
		FForceTarget ForceTarget;
		ForceTarget.bPosition = true;
		ForceTarget.Force = Force;
		ForceTarget.Position = Position;
		ForceTarget.bIsLocalForce = bIsLocalForce;

		FPhysTarget & TargetState = PhysTargetBuffers[External].FindOrAdd(Body);
		TargetState.Forces.Add(ForceTarget);
	}
}
void FPhysSubstepTask::AddTorque_AssumesLocked(FBodyInstance* Body, const FVector& Torque, bool bAccelChange)
{
	//We should only apply torque on non kinematic actors
	if (Body->IsNonKinematic())
	{
		SUBSTEPPING_WARNING()
		FTorqueTarget TorqueTarget;
		TorqueTarget.Torque = Torque;
		TorqueTarget.bAccelChange = bAccelChange;

		FPhysTarget & TargetState = PhysTargetBuffers[External].FindOrAdd(Body);
		TargetState.Torques.Add(TorqueTarget);
	}
}

void FPhysSubstepTask::ClearTorques_AssumesLocked(FBodyInstance* Body)
{
	if (Body->IsNonKinematic())
	{
		SUBSTEPPING_WARNING()
		FPhysTarget & TargetState = PhysTargetBuffers[External].FindOrAdd(Body);
		TargetState.Torques.Empty();
	}
}

void FPhysSubstepTask::AddRadialForceToBody_AssumesLocked(FBodyInstance* Body, const FVector& Origin, const float Radius, const float Strength, const uint8 Falloff, const bool bAccelChange)
{
	//We should only apply torque on non kinematic actors
	if (Body->IsNonKinematic())
	{
		SUBSTEPPING_WARNING()
		FRadialForceTarget RadialForceTarget;
		RadialForceTarget.Origin = Origin;
		RadialForceTarget.Radius = Radius;
		RadialForceTarget.Strength = Strength;
		RadialForceTarget.Falloff = Falloff;
		RadialForceTarget.bAccelChange = bAccelChange;

		FPhysTarget & TargetState = PhysTargetBuffers[External].FindOrAdd(Body);
		TargetState.RadialForces.Add(RadialForceTarget);
	}
}

void FPhysSubstepTask::ClearForces_AssumesLocked(FBodyInstance* Body)
{
	if (Body->IsNonKinematic())
	{
		SUBSTEPPING_WARNING()
		FPhysTarget & TargetState = PhysTargetBuffers[External].FindOrAdd(Body);
		TargetState.Forces.Empty();
		TargetState.RadialForces.Empty();
	}
}

/** Applies custom physics - Assumes caller has obtained writer lock */
void FPhysSubstepTask::ApplyCustomPhysics(const FPhysTarget& PhysTarget, FBodyInstance* BodyInstance, float DeltaTime)
{
	FSubstepCallbackGuard Guard(*this);
	for (int32 i = 0; i < PhysTarget.CustomPhysics.Num(); ++i)
	{
		const FCustomTarget& CustomTarget = PhysTarget.CustomPhysics[i];

		CustomTarget.CalculateCustomPhysics->ExecuteIfBound(DeltaTime, BodyInstance);
	}
}


/** Applies forces - Assumes caller has obtained writer lock */
void FPhysSubstepTask::ApplyForces_AssumesLocked(const FPhysTarget& PhysTarget, FBodyInstance* BodyInstance)
{
    check(false);
}

/** Applies torques - Assumes caller has obtained writer lock */
void FPhysSubstepTask::ApplyTorques_AssumesLocked(const FPhysTarget& PhysTarget, FBodyInstance* BodyInstance)
{
    check(false);
}

/** Applies radial forces - Assumes caller has obtained writer lock */
void FPhysSubstepTask::ApplyRadialForces_AssumesLocked(const FPhysTarget& PhysTarget, FBodyInstance* BodyInstance)
{
    check(false);
}


/** Interpolates kinematic actor transform - Assumes caller has obtained writer lock */
void FPhysSubstepTask::InterpolateKinematicActor_AssumesLocked(const FPhysTarget& PhysTarget, FBodyInstance* BodyInstance, float InAlpha)
{
    check(false);
}

void FPhysSubstepTask::SubstepInterpolation(float InAlpha, float DeltaTime)
{
}

float FPhysSubstepTask::UpdateTime(float UseDelta)
{
	float FrameRate = 1.f;
	uint32 MaxSubSteps = 1;

	UPhysicsSettings * PhysSetting = UPhysicsSettings::Get();
	FrameRate = PhysSetting->MaxSubstepDeltaTime;
	MaxSubSteps = PhysSetting->MaxSubsteps;
	
	float FrameRateInv = 1.f / FrameRate;

	//Figure out how big dt to make for desired framerate
	DeltaSeconds = FMath::Min(UseDelta, MaxSubSteps * FrameRate);
	NumSubsteps = FMath::CeilToInt(DeltaSeconds * FrameRateInv);
	NumSubsteps = FMath::Max(NumSubsteps > MaxSubSteps ? MaxSubSteps : NumSubsteps, (uint32) 1);
	SubTime = DeltaSeconds / NumSubsteps;

	return SubTime;
}

DECLARE_CYCLE_STAT(TEXT("Phys SubstepStart"), STAT_SubstepSimulationStart, STATGROUP_Physics);
DECLARE_CYCLE_STAT(TEXT("Phys SubstepEnd"), STAT_SubstepSimulationEnd, STATGROUP_Physics);

void FPhysSubstepTask::SubstepSimulationStart()
{
	SCOPE_CYCLE_COUNTER(STAT_TotalPhysicsTime);
	SCOPE_CYCLE_COUNTER(STAT_SubstepSimulationStart);
}

void FPhysSubstepTask::SubstepSimulationEnd(ENamedThreads::Type CurrentThread, const FGraphEventRef& MyCompletionGraphEvent)
{
}
