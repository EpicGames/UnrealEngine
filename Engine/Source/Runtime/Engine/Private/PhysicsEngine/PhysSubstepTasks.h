// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Async/TaskGraphInterfaces.h"
#include "CoreMinimal.h"
#include "EngineDefines.h"
#include "HAL/ThreadSafeBool.h"
#include "Physics/PhysicsInterfaceCore.h"
#include "PhysicsEngine/BodyInstance.h"

struct FSimulationScratchBuffer;

void FinishSceneStat();

/** Hold information about kinematic target */
struct FKinematicTarget
{
	FKinematicTarget() : BodyInstance(0) {}
	FKinematicTarget(FBodyInstance* Body, const FTransform& TM) :
		BodyInstance(Body),
		TargetTM(TM),
		OriginalTM(Body->GetUnrealWorldTransform(true, true))
	{
	}

	/** Kinematic actor we are setting target for*/
	FBodyInstance* BodyInstance;

	/** Target transform for kinematic actor*/
	FTransform TargetTM;

	/** Start transform for kinematic actor*/
	FTransform OriginalTM;
};

/** Kinematic target struct to use when lock is assumed */
struct FKinematicTarget_AssumesLocked : public FKinematicTarget
{
	FKinematicTarget_AssumesLocked(FBodyInstance* Body, const FTransform& TM)
	{
		BodyInstance = Body;
		TargetTM = TM;
		OriginalTM = Body->GetUnrealWorldTransform_AssumesLocked(true, true);
	}
};

/** Holds information about requested force */
struct FForceTarget
{
	FForceTarget(){}
	FForceTarget(const FVector& GivenForce) : Force(GivenForce), bPosition(false){}
	FForceTarget(const FVector& GivenForce, const FVector& GivenPosition) : Force(GivenForce), Position(GivenPosition), bPosition(true){}
	FVector Force;
	FVector Position;
	bool bPosition;
	bool bAccelChange;
	bool bIsLocalForce;
};

struct FTorqueTarget
{
	FTorqueTarget(){}
	FTorqueTarget(const FVector& GivenTorque) : Torque(GivenTorque){}
	FVector Torque;
	bool bAccelChange;
};

struct FRadialForceTarget
{
	FRadialForceTarget(){}
	FVector Origin;
	float Radius;
	float Strength;
	uint8 Falloff;
	bool bAccelChange;
};

struct FCustomTarget
{
	FCustomTarget(){ }
	FCustomTarget(const FCalculateCustomPhysics& GivenCalculateCustomPhysics) : CalculateCustomPhysics(&GivenCalculateCustomPhysics){}
	const FCalculateCustomPhysics* CalculateCustomPhysics;
};

/** Holds information on everything we need to fix for substepping of a single frame */
struct FPhysTarget
{
	FPhysTarget() : bKinematicTarget(false){}

	TArray<FForceTarget>  Forces;	//we can apply force at multiple places
	TArray<FTorqueTarget> Torques;
	TArray<FRadialForceTarget> RadialForces;
	TArray<FCustomTarget> CustomPhysics; //for calculating custom physics forces
	FKinematicTarget KinematicTarget;

	/** Tells us if the kinematic target has been set */
	bool bKinematicTarget;
	
	
};

/** Holds information used for substepping a scene */
class FPhysSubstepTask
{
public:
	void SetKinematicTarget_AssumesLocked(FBodyInstance* Body, const FTransform& TM);
	bool GetKinematicTarget_AssumesLocked(const FBodyInstance* Body, FTransform& OutTM) const;
	void AddCustomPhysics_AssumesLocked(FBodyInstance* Body, const FCalculateCustomPhysics& CalculateCustomPhysics);
	void AddForce_AssumesLocked(FBodyInstance* Body, const FVector& Force, bool bAccelChange);
	void AddForceAtPosition_AssumesLocked(FBodyInstance* Body, const FVector& Force, const FVector& Position, bool bIsLocalForce);
	void AddRadialForceToBody_AssumesLocked(FBodyInstance* Body, const FVector& Origin, const float Radius, const float Strength, const uint8 Falloff, const bool bAccelChange);
	void ClearForces_AssumesLocked(FBodyInstance* Body);
	void AddTorque_AssumesLocked(FBodyInstance* Body, const FVector& Torque, bool bAccelChange);
	void ClearTorques_AssumesLocked(FBodyInstance* Body);

	/** Removes a BodyInstance from doing substep work - should only be called when the FBodyInstance is getting destroyed */
	void RemoveBodyInstance_AssumesLocked(FBodyInstance* Body);

	
	void SwapBuffers();
	float UpdateTime(float UseDelta);

	void SubstepSimulationStart();
	void SubstepSimulationEnd(ENamedThreads::Type CurrentThread, const FGraphEventRef& MyCompletionGraphEvent);
		
private:

	/** Applies interpolation and forces on all needed actors*/
	void SubstepInterpolation(float Scale, float DeltaTime);
	void ApplyCustomPhysics(const FPhysTarget& PhysTarget, FBodyInstance* BodyInstance, float DeltaTime);
	void ApplyForces_AssumesLocked(const FPhysTarget& PhysTarget, FBodyInstance* BodyInstance);
	void ApplyTorques_AssumesLocked(const FPhysTarget& PhysTarget, FBodyInstance* BodyInstance);
	void ApplyRadialForces_AssumesLocked(const FPhysTarget& PhysTarget, FBodyInstance* BodyInstance);
	void InterpolateKinematicActor_AssumesLocked(const FPhysTarget& PhysTarget, FBodyInstance* BodyInstance, float Alpha);

	typedef TMap<FBodyInstance*, FPhysTarget> PhysTargetMap;
	PhysTargetMap PhysTargetBuffers[2];	//need to double buffer between physics thread and game thread
	uint32 NumSubsteps;
	float SubTime;
	float DeltaSeconds;
	FThreadSafeBool External;
	class  PhysXCompletionTask * FullSimulationTask;
	float Alpha;
	float StepScale;
	float TotalSubTime;
	uint32 CurrentSubStep;
	int32 SubstepCallbackGuard;
	friend struct FSubstepCallbackGuard;

	FGraphEventRef CompletionEvent;

	FPhysScene* PhysScene;
};