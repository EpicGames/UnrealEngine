// Copyright Epic Games, Inc. All Rights Reserved.

// Physics engine integration utilities

#include "Engine/World.h"
#include "Model.h"
#include "PBDRigidsSolver.h"
#include "Physics/Experimental/PhysScene_Chaos.h"
#include "PhysicsEngine/ConvexElem.h"
#include "PhysicsEngine/BodySetup.h"
#include "Chaos/TrackedGeometryManager.h"

/** 
 * Picks a snap distance for a UModel planes->verts conversion. To preserve PhysX behavior SMALL_NUMBER is used for its snap distance.
 * Chaos uses a scaled snap distance in relation to the model bounds to create hulls with less points.
 */
static float SuggestConvexVertexSnapDistance(UModel* InModel)
{
	// For an object around 1cmx1cmx1cm this is the vertex snap distance, then scaled to match the input model
	constexpr float UnitSphereSnapDistance = 0.0015f;
	return UnitSphereSnapDistance * InModel->Bounds.SphereRadius;
}

/** Returns false if ModelToHulls operation should halt because of vertex count overflow. */
static bool AddConvexPrim(FKAggregateGeom* OutGeom, TArray<FPlane> &Planes, UModel* InModel)
{
	// Add Hull.
	FKConvexElem NewConvex;

	// Because of precision, we use the original model verts as 'snap to' verts.
	TArray<FVector> SnapVerts;
	for(int32 k=0; k<InModel->Verts.Num(); k++)
	{
		// Find vertex vector. Bit of  hack - sometimes FVerts are uninitialised.
		const int32 PointIx = InModel->Verts[k].pVertex;
		if(PointIx < 0 || PointIx >= InModel->Points.Num())
		{
			continue;
		}

		SnapVerts.Add((FVector)InModel->Points[PointIx]);
	}

	// Create a hull from a set of planes
	bool bSuccess = NewConvex.HullFromPlanes(Planes, SnapVerts, SuggestConvexVertexSnapDistance(InModel));

	// If it failed for some reason, remove from the array
	if(bSuccess && NewConvex.ElemBox.IsValid)
	{
		OutGeom->ConvexElems.Add(NewConvex);
	}

	// Return if we succeeded or not
	return bSuccess;
}

// Worker function for traversing collision mode/blocking volumes BSP.
// At each node, we record, the plane at this node, and carry on traversing.
// We are interested in 'inside' ie solid leafs.
/** Returns false if ModelToHulls operation should halt because of vertex count overflow. */
static bool ModelToHullsWorker(FKAggregateGeom* outGeom,
								UModel* inModel, 
								int32 nodeIx, 
								bool bOutside, 
								TArray<FPlane> &planes)
{
	FBspNode& node = inModel->Nodes[nodeIx];
	// BACK
	if (node.iBack != INDEX_NONE) // If there is a child, recurse into it.
	{
		planes.Add(FPlane(node.Plane));
		if (!ModelToHullsWorker(outGeom, inModel, node.iBack, node.ChildOutside(0, bOutside), planes))
		{
			return false;
		}
		planes.RemoveAt(planes.Num() - 1);
	}
	else if (!node.ChildOutside(0, bOutside)) // If its a leaf, and solid (inside)
	{
		planes.Add(FPlane(node.Plane));
		if (!AddConvexPrim(outGeom, planes, inModel))
		{
			return false;
		}
		planes.RemoveAt(planes.Num() - 1);
	}

	// FRONT
	if (node.iFront != INDEX_NONE)
	{
		planes.Add(FPlane(node.Plane.Flip()));
		if (!ModelToHullsWorker(outGeom, inModel, node.iFront, node.ChildOutside(1, bOutside), planes))
		{
			return false;
		}
		planes.RemoveAt(planes.Num() - 1);
	}
	else if (!node.ChildOutside(1, bOutside))
	{
		planes.Add(FPlane(node.Plane.Flip()));
		if (!AddConvexPrim(outGeom, planes, inModel))
		{
			return false;
		}
		planes.RemoveAt(planes.Num() - 1);
	}

	return true;
}

bool UBodySetup::CreateFromModel(UModel* InModel, bool bRemoveExisting)
{
	if ( bRemoveExisting )
	{
		RemoveSimpleCollision();
	}

	const int32 NumHullsAtStart = AggGeom.ConvexElems.Num();
	
	bool bSuccess = false;

	if( InModel != NULL && InModel->Nodes.Num() > 0)
	{
		TArray<FPlane>	Planes;
		bSuccess = ModelToHullsWorker(&AggGeom, InModel, 0, InModel->RootOutside, Planes);
		if ( !bSuccess )
		{
			// ModelToHullsWorker failed.  Clear out anything that may have been created.
			AggGeom.ConvexElems.Empty();
		}
	}

	// Create new GUID
	InvalidatePhysicsData();
	return bSuccess;
}

//////////////////////////////////////////////////////////////////////////
// FRigidBodyCollisionInfo

void FRigidBodyCollisionInfo::SetFrom(const FBodyInstance* BodyInst, const FVector& InDeltaVelocity)
{
	if(BodyInst != NULL)
	{
		BodyIndex = BodyInst->InstanceBodyIndex;
		BoneName = BodyInst->BodySetup.IsValid() ? BodyInst->BodySetup->BoneName : NAME_None;
		DeltaVelocity = InDeltaVelocity;

		if(BodyInst->OwnerComponent.IsValid())
		{
			Component = BodyInst->OwnerComponent;
			Actor = Component->GetOwner();
		}
	}
	else
	{
		Component = NULL;
		Actor = NULL;
		BodyIndex = INDEX_NONE;
		BoneName = NAME_None;
		DeltaVelocity = FVector::ZeroVector;
	}
}


FBodyInstance* FRigidBodyCollisionInfo::GetBodyInstance() const
{
	FBodyInstance* BodyInst = NULL;
	if(Component.IsValid())
	{
		BodyInst = Component->GetBodyInstance(BoneName, true, BodyIndex);
	}
	return BodyInst;
}

//////////////////////////////////////////////////////////////////////////
// FCollisionNotifyInfo

bool FCollisionNotifyInfo::IsValidForNotify() const
{
	return (Info0.Component.IsValid() && Info1.Component.IsValid());
}

/** Iterate over ContactInfos array and swap order of information */
void FCollisionImpactData::SwapContactOrders()
{
	for(int32 i=0; i<ContactInfos.Num(); i++)
	{
		ContactInfos[i].SwapOrder();
	}
}

/** Swap the order of info in this info  */
void FRigidBodyContactInfo::SwapOrder()
{
	Swap(PhysMaterial[0], PhysMaterial[1]);
	ContactNormal = -ContactNormal;
}

//////////////////////////////////////////////////////////////////////////
// FCollisionResponseContainer

/** Set the status of a particular channel in the structure. */
bool FCollisionResponseContainer::SetResponse(ECollisionChannel Channel, ECollisionResponse NewResponse)
{
	if (Channel < UE_ARRAY_COUNT(EnumArray))
	{
		uint8& CurrentResponse = EnumArray[Channel];
		if (CurrentResponse != NewResponse)
		{
			CurrentResponse = NewResponse;
			return true;
		}
	}
	return false;
}

/** Set all channels to the specified state */
bool FCollisionResponseContainer::SetAllChannels(ECollisionResponse NewResponse)
{
	bool bHasChanged = false;
	for(int32 i=0; i<UE_ARRAY_COUNT(EnumArray); i++)
	{
		uint8& CurrentResponse = EnumArray[i];
		if (CurrentResponse != NewResponse)
		{
			CurrentResponse = NewResponse;
			bHasChanged = true;
		}
	}
	return bHasChanged;
}

bool FCollisionResponseContainer::ReplaceChannels(ECollisionResponse OldResponse, ECollisionResponse NewResponse)
{
	bool bHasChanged = false;
	for (int32 i = 0; i < UE_ARRAY_COUNT(EnumArray); i++)
	{
		uint8& CurrentResponse = EnumArray[i];
		if(CurrentResponse == OldResponse)
		{
			CurrentResponse = NewResponse;
			bHasChanged = true;
		}
	}
	return bHasChanged;
}

FCollisionResponseContainer FCollisionResponseContainer::CreateMinContainer(const FCollisionResponseContainer& A, const FCollisionResponseContainer& B)
{
	FCollisionResponseContainer Result;
	for(int32 i=0; i<UE_ARRAY_COUNT(Result.EnumArray); i++)
	{
		Result.EnumArray[i] = FMath::Min(A.EnumArray[i], B.EnumArray[i]);
	}
	return Result;
}


/** This constructor will zero out the struct */
FCollisionResponseContainer::FCollisionResponseContainer()
{
	// if this is called before profile is initialized, it will be overwritten by postload code
	// if this is called after profile is initialized, this will have correct values
	*this = FCollisionResponseContainer::DefaultResponseContainer;
}

FCollisionResponseContainer::FCollisionResponseContainer(ECollisionResponse DefaultResponse)
{
	SetAllChannels(DefaultResponse);
}

FCollisionEnabledMask::FCollisionEnabledMask(int8 InBits)
	: Bits(InBits)
{ }

FCollisionEnabledMask::FCollisionEnabledMask(ECollisionEnabled::Type CollisionEnabled)
	: Bits(1 << CollisionEnabled)
{ }

FCollisionEnabledMask::operator int8() const
{
	return Bits;
}

FCollisionEnabledMask::operator bool() const
{
	return Bits != 0;
}

FCollisionEnabledMask FCollisionEnabledMask::operator&(const FCollisionEnabledMask Other) const
{
	return Bits & Other.Bits;
}

FCollisionEnabledMask FCollisionEnabledMask::operator&(const ECollisionEnabled::Type Other) const
{
	return Bits & FCollisionEnabledMask(Other).Bits;
}

FCollisionEnabledMask FCollisionEnabledMask::operator|(const FCollisionEnabledMask Other) const
{
	return Bits | Other.Bits;
}

FCollisionEnabledMask FCollisionEnabledMask::operator|(const ECollisionEnabled::Type Other) const
{
	return Bits | FCollisionEnabledMask(Other).Bits;
}

FCollisionEnabledMask operator&(const ECollisionEnabled::Type A, const ECollisionEnabled::Type B)
{
	return FCollisionEnabledMask(A) & FCollisionEnabledMask(B);
}

FCollisionEnabledMask operator&(const ECollisionEnabled::Type A, const FCollisionEnabledMask B)
{
	return FCollisionEnabledMask(A) & B;
}

FCollisionEnabledMask operator|(const ECollisionEnabled::Type A, const ECollisionEnabled::Type B)
{
	return FCollisionEnabledMask(A) | FCollisionEnabledMask(B);
}

FCollisionEnabledMask operator|(const ECollisionEnabled::Type A, const FCollisionEnabledMask B)
{
	return FCollisionEnabledMask(A) | B;
}

bool FPhysScene::ExecPxVis(uint32 SceneType, const TCHAR* Cmd, FOutputDevice* Ar)
{
    return false;
}

bool FPhysScene::ExecApexVis(uint32 SceneType, const TCHAR* Cmd, FOutputDevice* Ar)
{
    return false;
}

bool FPhysicsInterface::ExecPhysCommands(const TCHAR* Cmd, FOutputDevice* OutputDevice, UWorld* InWorld)
{
	if (FParse::Command(&Cmd, TEXT("ChaosGeometryMemory")))
	{
		Chaos::FTrackedGeometryManager::Get().DumpMemoryUsage(Cmd, OutputDevice);
		return true;
	}

	uint32 NumFrames = 0;
	if(FParse::Value(Cmd,TEXT("ChaosRewind"), NumFrames))
	{
		InWorld->GetPhysicsScene()->ResimNFrames(NumFrames);
		return true;
	}
#if CHAOS_MEMORY_TRACKING
	if (FParse::Command(&Cmd, TEXT("ChaosMemoryDistribution")))
	{
		class FChaosArchiveExternalContext : public Chaos::FChaosArchive
		{
		public:
			FChaosArchiveExternalContext(FArchive& ArIn)
				: FChaosArchive(ArIn)
			{
			}

			explicit FChaosArchiveExternalContext(FArchive& ArIn, const TSharedRef<Chaos::FChaosArchiveContext>& InExternalSharedContext)
				: FChaosArchive(ArIn, InExternalSharedContext)
			{
			}
		};

		//
		// NOTE: This is an awful, awful way to do this.
		//

		// Make an archive and serialze the whole scene.
		// TODO: Don't do this!
		FArchive BaseAr;
		BaseAr.SetIsLoading(false);
		BaseAr.SetIsSaving(true);
		TSharedRef<Chaos::FChaosArchiveContext> ArchiveContext = MakeShared<Chaos::FChaosArchiveContext>();
		FChaosArchiveExternalContext Ar(BaseAr, ArchiveContext);
		FPhysScene* PhysScene = InWorld->GetPhysicsScene();
		Chaos::FPhysicsSolver* Solver = PhysScene->GetSolver();
		Chaos::FPBDRigidsEvolution* Evolution = Solver->GetEvolution();
		Evolution->Serialize(Ar);

		// Grab the memory tracking map from the archive context
		const TMap<FName, Chaos::FChaosArchiveSectionData>& MemoryMap = ArchiveContext->SectionMap;

		OutputDevice->Logf(TEXT("Chaos serialized memory distribution:"));
		int64 TotalBytes = 0;
		for (const TPair<FName, Chaos::FChaosArchiveSectionData>& It : MemoryMap)
		{
			const FName SectionName = It.Key;
			const Chaos::FChaosArchiveSectionData SectionData = It.Value;
			TotalBytes += SectionData.SizeExclusive;
			OutputDevice->Logf(TEXT("%s ~ count: %d, bytes: %d, megabytes: %f"), *SectionName.ToString(), SectionData.Count, SectionData.SizeExclusive, (float)SectionData.SizeExclusive * 10e-7f);
		}
		OutputDevice->Logf(TEXT("Total bytes: %d, megabytes: %f"), TotalBytes, (float)TotalBytes * 10e-7f);
	}
#endif

    return false;
}
