// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	PhysXSupport.h: PhysX support
=============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "Stats/Stats.h"
#include "Misc/ScopeLock.h"
#include "EngineDefines.h"
#include "Containers/Queue.h"
#include "Physics/PhysicsFiltering.h"
#include "PhysXPublic.h"
#include "PhysXSupportCore.h"

class UBodySetup;
class UPhysicalMaterial;
struct FCollisionShape;
