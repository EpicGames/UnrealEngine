// Copyright Epic Games, Inc. All Rights Reserved.

#include "PhysicsEngine/PhysicalAnimationComponent.h"
#include "Chaos/ImplicitFwd.h"
#include "PrimitiveDrawingUtils.h"
#include "Components/SkeletalMeshComponent.h"
#include "Engine/SkeletalMesh.h"
#include "Physics/Experimental/PhysScene_Chaos.h"
#include "PhysicsEngine/PhysicsAsset.h"
#include "PhysicsEngine/SkeletalBodySetup.h"
#include "Chaos/ChaosConstraintSettings.h"
#include "Chaos/Sphere.h"
#include "PhysicsEngine/ConstraintInstance.h"
#include "PhysicsProxy/SingleParticlePhysicsProxy.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PhysicalAnimationComponent)

const FConstraintProfileProperties UPhysicalAnimationComponent::PhysicalAnimationProfile = []()
{
	//Setup the default constraint profile for all joints created by physical animation system
	FConstraintProfileProperties RetProfile;
	RetProfile.LinearLimit.XMotion = LCM_Free;
	RetProfile.LinearLimit.YMotion = LCM_Free;
	RetProfile.LinearLimit.ZMotion = LCM_Free;

	RetProfile.ConeLimit.Swing1Motion = ACM_Free;
	RetProfile.ConeLimit.Swing2Motion = ACM_Free;
	RetProfile.TwistLimit.TwistMotion = ACM_Free;

	RetProfile.LinearDrive.XDrive.bEnablePositionDrive = true;
	RetProfile.LinearDrive.XDrive.bEnableVelocityDrive = true;
	RetProfile.LinearDrive.YDrive.bEnablePositionDrive = true;
	RetProfile.LinearDrive.YDrive.bEnableVelocityDrive = true;
	RetProfile.LinearDrive.ZDrive.bEnablePositionDrive = true;
	RetProfile.LinearDrive.ZDrive.bEnableVelocityDrive = true;

	RetProfile.AngularDrive.SlerpDrive.bEnablePositionDrive = true;
	RetProfile.AngularDrive.SlerpDrive.bEnableVelocityDrive = true;
	RetProfile.AngularDrive.AngularDriveMode = EAngularDriveMode::SLERP;

	return RetProfile;
}();

UPhysicalAnimationComponent::UPhysicalAnimationComponent(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bWantsInitializeComponent = true;
	PrimaryComponentTick.bCanEverTick = true;
	PrimaryComponentTick.bTickEvenWhenPaused = true;
	PrimaryComponentTick.TickGroup = TG_PrePhysics;
	bPhysicsEngineNeedsUpdating = false;

	StrengthMultiplyer = 1.f;
}

void UPhysicalAnimationComponent::InitializeComponent()
{
	Super::InitializeComponent();
	InitComponent();
}

void UPhysicalAnimationComponent::InitComponent()
{
	if (SkeletalMeshComponent)
	{
		OnTeleportDelegateHandle = SkeletalMeshComponent->RegisterOnTeleportDelegate(FOnSkelMeshTeleported::CreateUObject(this, &UPhysicalAnimationComponent::OnTeleport));
		PrimaryComponentTick.AddPrerequisite(SkeletalMeshComponent, SkeletalMeshComponent->PrimaryComponentTick);
		UpdatePhysicsEngine();
	}
}

void UPhysicalAnimationComponent::BeginDestroy()
{
	if(SkeletalMeshComponent && OnTeleportDelegateHandle.IsValid())
	{
		SkeletalMeshComponent->UnregisterOnTeleportDelegate(OnTeleportDelegateHandle);
	}

	ReleasePhysicsEngine();
	Super::BeginDestroy();
}

void UPhysicalAnimationComponent::SetSkeletalMeshComponent(USkeletalMeshComponent* InSkeletalMeshComponent)
{
	if(SkeletalMeshComponent && OnTeleportDelegateHandle.IsValid())
	{
		SkeletalMeshComponent->UnregisterOnTeleportDelegate(OnTeleportDelegateHandle);
	}

	SkeletalMeshComponent = InSkeletalMeshComponent;
	DriveData.Empty();
	ReleasePhysicsEngine();
	InitComponent();

}

bool UpdatePhysicalAnimationSettings(FName BodyName, const FPhysicalAnimationData& InPhysicalAnimationData, TArray<FPhysicalAnimationData>& DriveData, const UPhysicsAsset& PhysAsset)
{
	int32 BodyIdx = PhysAsset.FindBodyIndex(BodyName);
	if (BodyIdx != INDEX_NONE)
	{
		//This code does a linear search in the insertion. This is by choice so that during tick we get nice tight cache for iteration. We could keep a map of sorts, but expected number of bodies is small
		FPhysicalAnimationData* UpdateAnimationData = DriveData.FindByPredicate([BodyName](const FPhysicalAnimationData& Elem) { return Elem.BodyName == BodyName; });
		if (UpdateAnimationData == nullptr)
		{
			UpdateAnimationData = &DriveData[DriveData.AddUninitialized()];
		}
		*UpdateAnimationData = InPhysicalAnimationData;
		UpdateAnimationData->BodyName = BodyName;

		return true;
	}

	return false;
}

void UPhysicalAnimationComponent::ApplyPhysicalAnimationSettings(FName BodyName, const FPhysicalAnimationData& PhysicalAnimationData)
{
	UPhysicsAsset* PhysAsset = SkeletalMeshComponent ? SkeletalMeshComponent->GetPhysicsAsset() : nullptr;
	if (PhysAsset)
	{
		if(UpdatePhysicalAnimationSettings(BodyName, PhysicalAnimationData, DriveData, *PhysAsset))
		{
			UpdatePhysicsEngine();
		}
	}
}

void UPhysicalAnimationComponent::ApplyPhysicalAnimationSettingsBelow(FName BodyName, const FPhysicalAnimationData& PhysicalAnimationData, bool bIncludeSelf)
{
	UPhysicsAsset* PhysAsset = SkeletalMeshComponent ? SkeletalMeshComponent->GetPhysicsAsset() : nullptr;
	if (PhysAsset && SkeletalMeshComponent)
	{
		TArray<FPhysicalAnimationData>& NewDriveData = DriveData;
		bool bNeedsUpdating = false;
		SkeletalMeshComponent->ForEachBodyBelow(BodyName, bIncludeSelf, /*bSkipCustomType=*/false, [PhysAsset, &NewDriveData, PhysicalAnimationData, &bNeedsUpdating](const FBodyInstance* BI)
		{
			const FName IterBodyName = PhysAsset->SkeletalBodySetups[BI->InstanceBodyIndex]->BoneName;
			bNeedsUpdating |= UpdatePhysicalAnimationSettings(IterBodyName, PhysicalAnimationData, NewDriveData, *PhysAsset);
		});

		if(bNeedsUpdating)
		{
			UpdatePhysicsEngine();
		}
	}
}


void UPhysicalAnimationComponent::ApplyPhysicalAnimationProfileBelow(FName BodyName, FName ProfileName, bool bIncludeSelf, bool bClearNotFound)
{
	UPhysicsAsset* PhysAsset = SkeletalMeshComponent ? SkeletalMeshComponent->GetPhysicsAsset() : nullptr;
	if (PhysAsset && SkeletalMeshComponent)
	{
		TArray<FPhysicalAnimationData>& NewDriveData = DriveData;
		bool bNeedsUpdating = false;
		SkeletalMeshComponent->ForEachBodyBelow(BodyName, bIncludeSelf, /*bSkipCustomType=*/false, [bClearNotFound, ProfileName, PhysAsset, &NewDriveData, &bNeedsUpdating](const FBodyInstance* BI)
		{
			if(USkeletalBodySetup* BodySetup = Cast<USkeletalBodySetup>(BI->BodySetup.Get()))
			{
				const FName IterBodyName = PhysAsset->SkeletalBodySetups[BI->InstanceBodyIndex]->BoneName;
				if(FPhysicalAnimationProfile* Profile = BodySetup->FindPhysicalAnimationProfile(ProfileName))
				{
					bNeedsUpdating |= UpdatePhysicalAnimationSettings(IterBodyName, Profile->PhysicalAnimationData, NewDriveData, *PhysAsset);
				}
				else if(bClearNotFound)
				{
					bNeedsUpdating |= UpdatePhysicalAnimationSettings(IterBodyName, FPhysicalAnimationData(), NewDriveData, *PhysAsset);
				}
			}
			
		});

		if (bNeedsUpdating)
		{
			UpdatePhysicsEngine();
		}
	}
}

FTransform UPhysicalAnimationComponent::GetBodyTargetTransform(FName BodyName) const
{
	if (SkeletalMeshComponent)
	{
		for (int32 DataIdx = 0; DataIdx < DriveData.Num(); ++DataIdx)
		{
			const FPhysicalAnimationData& PhysAnimData = DriveData[DataIdx];
			const FPhysicalAnimationInstanceData& InstanceData = RuntimeInstanceData[DataIdx];
			if (BodyName == PhysAnimData.BodyName)
			{
				if (FPhysicsActorHandle TargetActor = InstanceData.TargetActor)
				{
					// TODO: If kinematic targets implemented, fetch target and don't use position.
					return FTransform(TargetActor->GetGameThreadAPI().R(), TargetActor->GetGameThreadAPI().X());
				}

				break;
			}
		}

		// if body isn't controlled by physical animation, just return the body position
		const TArray<FTransform>& ComponentSpaceTransforms = SkeletalMeshComponent->GetComponentSpaceTransforms();
		const int32 BoneIndex = SkeletalMeshComponent->GetBoneIndex(BodyName);
		if (ComponentSpaceTransforms.IsValidIndex(BoneIndex))
		{
			return ComponentSpaceTransforms[BoneIndex] * SkeletalMeshComponent->GetComponentToWorld();
		}
	}

	return FTransform::Identity;
}

FTransform ComputeWorldSpaceTargetTM(
	const USkeletalMeshComponent& SkeletalMeshComponent, 
	const TArrayView<const FTransform> ComponentSpaceTransforms, 
	int32 BoneIndex)
{
	return ComponentSpaceTransforms[BoneIndex] * SkeletalMeshComponent.GetComponentToWorld();
}

FTransform ComputeLocalSpaceTargetTM(
	const USkeletalMeshComponent& SkeletalMeshComponent, 
	const UPhysicsAsset& PhysAsset, 
	const TArrayView<const FTransform> BoneSpaceTransforms,
	int32 BoneIndex)
{
	const FReferenceSkeleton& RefSkeleton = SkeletalMeshComponent.GetSkeletalMeshAsset()->GetRefSkeleton();
	FTransform AccumulatedDelta = BoneSpaceTransforms[BoneIndex];
	int32 CurBoneIdx = BoneIndex;
	while ((CurBoneIdx = RefSkeleton.GetParentIndex(CurBoneIdx)) != INDEX_NONE)
	{
		FName BoneName = RefSkeleton.GetBoneName(CurBoneIdx);
		int32 BodyIndex = PhysAsset.FindBodyIndex(BoneName);

		if (CurBoneIdx == BoneIndex)	//some kind of loop so just stop TODO: warn?
		{
			break;
		}

		if (SkeletalMeshComponent.Bodies.IsValidIndex(BodyIndex))
		{
			if (BodyIndex < SkeletalMeshComponent.Bodies.Num())
			{
				FBodyInstance* ParentBody = SkeletalMeshComponent.Bodies[BodyIndex];
				const FTransform NewWorldTM = AccumulatedDelta * ParentBody->GetUnrealWorldTransform_AssumesLocked();
				return NewWorldTM;
			}
			else
			{
				// Bodies array has changed on us?
				break;
			}
		}

		AccumulatedDelta = AccumulatedDelta * BoneSpaceTransforms[CurBoneIdx];
	}

	return FTransform::Identity;
}

FTransform ComputeTargetTM(
	const FPhysicalAnimationData& PhysAnimData, 
	const USkeletalMeshComponent& SkeletalMeshComponent, 
	const UPhysicsAsset& PhysAsset, 
	const TArrayView<const FTransform> BoneSpaceTransforms,
	const TArrayView<const FTransform> ComponentSpaceTransforms, 
	int32 BoneIndex)
{
	return PhysAnimData.bIsLocalSimulation 
		? ComputeLocalSpaceTargetTM(SkeletalMeshComponent, PhysAsset, BoneSpaceTransforms, BoneIndex)
		: ComputeWorldSpaceTargetTM(SkeletalMeshComponent, ComponentSpaceTransforms, BoneIndex);
}

void UPhysicalAnimationComponent::UpdateTargetActors(ETeleportType TeleportType)
{
	UPhysicsAsset* PhysAsset = SkeletalMeshComponent ? SkeletalMeshComponent->GetPhysicsAsset() : nullptr;
	if (PhysAsset && SkeletalMeshComponent->GetSkeletalMeshAsset())
	{
		const FReferenceSkeleton& RefSkeleton = SkeletalMeshComponent->GetSkeletalMeshAsset()->GetRefSkeleton();

		// Note we use GetEditableComponentSpaceTransforms because we need to update target actors in the midst of the 
		// various anim ticks, before buffers are flipped (which happens in the skel mesh component's post-physics tick)
		const TArrayView<const FTransform> ComponentSpaceTransforms = SkeletalMeshComponent->GetEditableComponentSpaceTransforms();

		if (ComponentSpaceTransforms.IsEmpty())
		{
			// Our tick depends on the skeletal mesh primary tick. When physics is disabled (i.e.
			// physics blend = 0), this might have been allowed to run post-physics. Thus, if
			// physics blending is enabled after the ticks have started processing (this can happen
			// from the actor's Blueprint code) then for one tick we might find ourselves running in
			// a post-physics tick group.This can result in this function running at the same time
			// as the secondary EndPhysics SKM tick function that performs a parallel blend back
			// from physics into the SKM component transforms. When this happens, the component
			// transforms will have been moved temporarily into the animation context. Preventing
			// this from happening is difficult (as it would need the tick functions to be updated
			// whilst they are getting processed) so we simply return if the component transforms
			// are missing. Note that setting CVarAnimationDelaysEndGroup
			// ("tick.AnimationDelaysEndGroup") to 0 should prevent us from getting here, as it
			// stops the primary animation tick from being allowed to run after PrePhysics.
			return;
		}
		FPhysicsCommand::ExecuteWrite(SkeletalMeshComponent, [&]()
		{
			const TArrayView<const FTransform> BoneSpaceTransforms = SkeletalMeshComponent->GetBoneSpaceTransformsView();
			for (int32 DataIdx = 0; DataIdx < DriveData.Num(); ++DataIdx)
			{
				const FPhysicalAnimationData& PhysAnimData = DriveData[DataIdx];
				FPhysicalAnimationInstanceData& InstanceData = RuntimeInstanceData[DataIdx];

				if (FPhysicsActorHandle TargetActor = InstanceData.TargetActor)
				{
					const int32 BoneIdx = RefSkeleton.FindBoneIndex(PhysAnimData.BodyName);
					// It's possible the skeletal mesh has changed out from under us. In that case
					// we should probably reset, but at the very least don't do work on non-existent bones
					if (BoneIdx != INDEX_NONE)
					{
						const FTransform TargetTM = ComputeTargetTM(
							PhysAnimData, *SkeletalMeshComponent, *PhysAsset, 
							BoneSpaceTransforms, ComponentSpaceTransforms, BoneIdx);
						// todo(chaos): Calling SetKinematicTarget_AssumesLocked before
						// SetGlobalPose_AssumesLocked is unnecessary for chaos. We should fix this
						// when PhysX is removed.
						FPhysicsInterface::SetKinematicTarget_AssumesLocked(TargetActor, TargetTM);

						if (TeleportType == ETeleportType::TeleportPhysics)
						{
							FPhysicsInterface::SetGlobalPose_AssumesLocked(TargetActor, TargetTM);
						}
					}
				}
			}
		});
	}
}

void UPhysicalAnimationComponent::OnTeleport()
{
	if (bPhysicsEngineNeedsUpdating)
	{
		UpdatePhysicsEngineImp();
	}

	UpdateTargetActors(ETeleportType::TeleportPhysics);
}

void UPhysicalAnimationComponent::TickComponent(float DeltaTime, enum ELevelTick TickType, FActorComponentTickFunction *ThisTickFunction)
{
	if (bPhysicsEngineNeedsUpdating)
	{
		UpdatePhysicsEngineImp();
	}

	UpdateTargetActors(ETeleportType::None);
}

void SetMotorStrength(FConstraintInstance& ConstraintInstance, const FPhysicalAnimationData& PhysAnimData, float StrengthMultiplier)
{
	float PositionStrengthMultiplier = StrengthMultiplier;
	float VelocityStrengthMultiplier = StrengthMultiplier;
	float OrientationStrengthMultiplier = StrengthMultiplier;
	float AngularVelocityStrengthMultiplier = StrengthMultiplier;

	PositionStrengthMultiplier *= Chaos::ConstraintSettings::LinearDriveStiffnessScale();
	VelocityStrengthMultiplier *= Chaos::ConstraintSettings::LinearDriveDampingScale();
	OrientationStrengthMultiplier *= Chaos::ConstraintSettings::AngularDriveStiffnessScale();
	AngularVelocityStrengthMultiplier *= Chaos::ConstraintSettings::AngularDriveDampingScale();

	ConstraintInstance.SetAngularDriveParams(
		PhysAnimData.OrientationStrength * OrientationStrengthMultiplier, 
		PhysAnimData.AngularVelocityStrength * AngularVelocityStrengthMultiplier,
		PhysAnimData.MaxAngularForce * StrengthMultiplier
	);

	if (PhysAnimData.bIsLocalSimulation)	//linear only works for world space simulation
	{
		ConstraintInstance.SetLinearDriveParams(0.f, 0.f, 0.f);
	}
	else
	{
		ConstraintInstance.SetLinearDriveParams(
			PhysAnimData.PositionStrength * PositionStrengthMultiplier, 
			PhysAnimData.VelocityStrength * VelocityStrengthMultiplier, 
			PhysAnimData.MaxLinearForce * StrengthMultiplier
		);
	}
}

void UPhysicalAnimationComponent::UpdatePhysicsEngine()
{
	if (SkeletalMeshComponent)
	{
		SkeletalMeshComponent->WakeAllRigidBodies();
	}
	bPhysicsEngineNeedsUpdating = true;	//must defer until tick so that animation can finish
}

void UPhysicalAnimationComponent::UpdatePhysicsEngineImp()
{
	bPhysicsEngineNeedsUpdating = false;
	UPhysicsAsset* PhysAsset = SkeletalMeshComponent ? SkeletalMeshComponent->GetPhysicsAsset() : nullptr;
	if(PhysAsset && SkeletalMeshComponent->GetSkeletalMeshAsset())
	{
		//TODO: This is hacky and assumes constraints can only be added and not removed. True for now, but bad in general!
		const int32 NumData = DriveData.Num();
		const int32 NumInstances = RuntimeInstanceData.Num();
		
		RuntimeInstanceData.AddZeroed(NumData - NumInstances);

		// Note we use GetEditableComponentSpaceTransforms because we need to update target actors in the midst of the 
		// various anim ticks, before buffers are flipped (which happens in the skel mesh component's post-physics tick)
		const TArray<FTransform>& SpaceBases = SkeletalMeshComponent->GetEditableComponentSpaceTransforms();
		const FReferenceSkeleton& RefSkeleton = SkeletalMeshComponent->GetSkeletalMeshAsset()->GetRefSkeleton();

		FPhysicsCommand::ExecuteWrite(SkeletalMeshComponent, [&]()
		{
			TArray<FTransform> LocalTransforms = SkeletalMeshComponent->GetBoneSpaceTransforms();

			for (int32 DataIdx = 0; DataIdx < DriveData.Num(); ++DataIdx)
			{
				bool bNewConstraint = false;
				const FPhysicalAnimationData& PhysAnimData = DriveData[DataIdx];
				FPhysicalAnimationInstanceData& InstanceData = RuntimeInstanceData[DataIdx];
				FConstraintInstance*& ConstraintInstance = InstanceData.ConstraintInstance;
				if(ConstraintInstance == nullptr)
				{
					bNewConstraint = true;
					ConstraintInstance = new FConstraintInstance();
					ConstraintInstance->ProfileInstance = PhysicalAnimationProfile;
				}

				//Apply drive forces
				SetMotorStrength(*ConstraintInstance, PhysAnimData, StrengthMultiplyer);
				
				if(bNewConstraint)
				{
					//Find body instances
					int32 ChildBodyIdx = PhysAsset->FindBodyIndex(PhysAnimData.BodyName);
					if (FBodyInstance* ChildBody = (ChildBodyIdx == INDEX_NONE ? nullptr : SkeletalMeshComponent->Bodies[ChildBodyIdx]))
					{
						if (FPhysicsActorHandle ActorHandle = ChildBody->GetPhysicsActor())
						{
							FPhysScene* Scene = ChildBody->GetPhysicsScene();

							ConstraintInstance->SetRefFrame(EConstraintFrame::Frame1, FTransform::Identity);
							ConstraintInstance->SetRefFrame(EConstraintFrame::Frame2, FTransform::Identity);

							const FTransform TargetTM = ComputeTargetTM(PhysAnimData, *SkeletalMeshComponent, *PhysAsset, LocalTransforms, SpaceBases, ChildBody->InstanceBoneIndex);

							// Create kinematic actor to attach to joint
							FPhysicsActorHandle KineActor;
							FActorCreationParams Params;
							Params.bSimulatePhysics = false;
							Params.bQueryOnly = false;
							Params.bUpdateKinematicFromSimulation = false;
							Params.Scene = Scene;
							Params.bStatic = false;
							Params.InitialTM = TargetTM;
							FPhysicsInterface::CreateActor(Params, KineActor);
							
							// Chaos requires our particles have geometry.
							auto Sphere = MakeImplicitObjectPtr<Chaos::FImplicitSphere3>(FVector(0,0,0), 0);
							KineActor->GetGameThreadAPI().SetGeometry(Sphere);

							KineActor->GetGameThreadAPI().SetUserData(nullptr);

							TArray<FPhysicsActorHandle> ActorHandles({ KineActor });
							Scene->AddActorsToScene_AssumesLocked(ActorHandles, /*bImmediate=*/false);

							// Save reference to the kinematic actor.
							InstanceData.TargetActor = KineActor;

							ConstraintInstance->InitConstraint_AssumesLocked(ChildBody->GetPhysicsActor(), InstanceData.TargetActor, 1.f);
						}
					}
				}
			}
		});
	}
}

void UPhysicalAnimationComponent::SetStrengthMultiplyer(float InStrengthMultiplyer)
{
	if(InStrengthMultiplyer >= 0.f)
	{
		StrengthMultiplyer = InStrengthMultiplyer;

		FPhysicsCommand::ExecuteWrite(SkeletalMeshComponent, [&]()
		{
			for (int32 DataIdx = 0; DataIdx < DriveData.Num(); ++DataIdx)
			{
				bool bNewConstraint = false;
				const FPhysicalAnimationData& PhysAnimData = DriveData[DataIdx];
				//added guard around crashing animation dereference
				if (DataIdx < RuntimeInstanceData.Num())
				{
					FPhysicalAnimationInstanceData& InstanceData = RuntimeInstanceData[DataIdx];
					if (FConstraintInstance* ConstraintInstance = InstanceData.ConstraintInstance)
					{
						//Apply drive forces
						SetMotorStrength(*ConstraintInstance, PhysAnimData, StrengthMultiplyer);
					}
				}
			}
		});
	}
}

void UPhysicalAnimationComponent::ReleasePhysicsEngine()
{
	// #PHYS2 On shutdown, SkelMeshComp is null, so we can't lock using that, need to lock based on scene from body
	//FPhysicsCommand::ExecuteWrite(SkeletalMeshComponent, [&]()
	//{
		for(FPhysicalAnimationInstanceData& Instance : RuntimeInstanceData)
		{
			if(Instance.ConstraintInstance)
			{
				Instance.ConstraintInstance->TermConstraint();
				delete Instance.ConstraintInstance;
				Instance.ConstraintInstance = nullptr;
			}

			if(Instance.TargetActor)
			{
				FChaosScene* PhysScene = FChaosEngineInterface::GetCurrentScene(Instance.TargetActor);
				if (ensure(PhysScene))
				{
					FPhysInterface_Chaos::ReleaseActor(Instance.TargetActor, PhysScene);
				}
				Instance.TargetActor = nullptr;
			}
		}

		RuntimeInstanceData.Reset();
	//});
}

#if WITH_EDITOR
static const FColor	TargetActorColor(255, 0, 0);

void UPhysicalAnimationComponent::DebugDraw(FPrimitiveDrawInterface* PDI) const
{
	for (const FPhysicalAnimationInstanceData& PhysAnimData : RuntimeInstanceData)
	{
		if (PhysAnimData.TargetActor)
		{
			PDI->DrawPoint(PhysAnimData.TargetActor->GetGameThreadAPI().X(), TargetActorColor, 3.f, SDPG_World);
		}
	}
}
#endif

