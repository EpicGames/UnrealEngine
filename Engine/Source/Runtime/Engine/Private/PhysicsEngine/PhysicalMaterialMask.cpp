// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	PhysicalMaterial.cpp
=============================================================================*/ 

#include "PhysicalMaterials/PhysicalMaterialMask.h"
#include "Physics/Experimental/PhysInterface_Chaos.h"
#include "UObject/UnrealType.h"

#if WITH_EDITOR

#include "AssetRegistry/AssetData.h"
#include "EditorFramework/AssetImportData.h"
#include "Modules/ModuleManager.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "Engine/Texture.h"

#endif

#include "Chaos/PhysicalMaterials.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PhysicalMaterialMask)

DEFINE_LOG_CATEGORY_STATIC(LogPhysicalMaterialMask, Log, All);

#if WITH_EDITOR

#include "HAL/IConsoleManager.h"

static void OnDumpPhysicalMaterialMaskData(const TArray< FString >& Arguments)
{
	bool bPhysMaterialMaskFound = false;
	if (Arguments.Num() > 0)
	{
		const FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));
		FAssetData PhysMatMaskAsset = AssetRegistryModule.Get().GetAssetByObjectPath(FSoftObjectPath(Arguments[0]));
		if (PhysMatMaskAsset.IsValid() == false)
		{
			TArray<FAssetData> AssetsInPackage;
			AssetRegistryModule.Get().GetAssetsByPackageName(*Arguments[0], AssetsInPackage);
			if (AssetsInPackage.Num() == 1)
			{
				PhysMatMaskAsset = AssetsInPackage[0];
			}
		}
		if (PhysMatMaskAsset.IsValid())
		{
			if (UPhysicalMaterialMask* PhysMatMask = Cast<UPhysicalMaterialMask>(PhysMatMaskAsset.GetAsset()))
			{
				PhysMatMask->DumpMaskData();
			}
			else
			{
				UE_LOGF(LogPhysicalMaterialMask, Warning, "Could not load PhysicalMaterialMask asset for argument: %ls", *Arguments[0]);
			}
		}
		else
		{
			UE_LOGF(LogPhysicalMaterialMask, Warning, "Could not find PhysicalMaterialMask asset for argument: %ls", *Arguments[0]);
		}
	}
	else
	{
		UE_LOGF(LogPhysicalMaterialMask, Warning, "Command requires a PhysicalMaterialMask asset reference to be specified.");
	}
}

static FAutoConsoleCommand CmdDumpPhysicalMaterialMaskData(
	TEXT("p.DumpPhysicalMaterialMaskData"),
	TEXT("Outputs the current mask data for the specified physical material mask asset to the log."),
	FConsoleCommandWithArgsDelegate::CreateStatic(&OnDumpPhysicalMaterialMaskData)
);

#endif // #if WITH_EDITOR

uint32 UPhysicalMaterialMask::INVALID_MASK_INDEX = 0xF;

UPhysicalMaterialMask::UPhysicalMaterialMask(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
#if WITH_EDITORONLY_DATA	
	,AssetImportData(nullptr)
	,MaskTexture(nullptr)
#endif
	,UVChannelIndex(0)
{
}

UPhysicalMaterialMask::UPhysicalMaterialMask(FVTableHelper& Helper)
	: Super(Helper)
{
}

UPhysicalMaterialMask::~UPhysicalMaterialMask() = default;

#if WITH_EDITOR

void UPhysicalMaterialMask::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	if (MaterialMaskHandle && MaterialMaskHandle->IsValid())
	{
		FPhysicsInterface::UpdateMaterialMask(*MaterialMaskHandle, this);
	}

	Super::PostEditChangeProperty(PropertyChangedEvent);
}

#endif // WITH_EDITOR

void UPhysicalMaterialMask::PostInitProperties()
{
#if WITH_EDITORONLY_DATA
	if (!HasAnyFlags(RF_ClassDefaultObject | RF_NeedLoad))
	{
		AssetImportData = NewObject<UAssetImportData>(this, TEXT("AssetImportData"));
		AssetImportData->AddFileName(FString(), 0); // add empty filename for now
	}
#endif
	Super::PostInitProperties();
}

void UPhysicalMaterialMask::PostLoad()
{
	Super::PostLoad();

#if WITH_EDITORONLY_DATA
	if (AssetImportData == nullptr)
	{
		AssetImportData = NewObject<UAssetImportData>(this, TEXT("AssetImportData"));
	}
#endif
}

void UPhysicalMaterialMask::FinishDestroy()
{
	if(MaterialMaskHandle)
	{
		FPhysicsInterface::ReleaseMaterialMask(*MaterialMaskHandle);
	}

	Super::FinishDestroy();
}


#if WITH_EDITOR

void UPhysicalMaterialMask::SetMaskTexture(UTexture* InMaskTexture, const FString& InTextureFilename)
{
	Modify();

	if (!AssetImportData)
	{
		AssetImportData = NewObject<UAssetImportData>(this, TEXT("AssetImportData"));
	}
	MaskTexture = InMaskTexture;
	AssetImportData->AddFileName(InTextureFilename, 0);

	FProperty* ParamProperty = FindFProperty<FProperty>(UPhysicalMaterialMask::StaticClass(), GET_MEMBER_NAME_STRING_CHECKED(UPhysicalMaterialMask, MaskTexture));
	FPropertyChangedEvent TextureChangedEvent(ParamProperty);
	PostEditChangeProperty(TextureChangedEvent);

	ParamProperty = FindFProperty<FProperty>(UPhysicalMaterialMask::StaticClass(), GET_MEMBER_NAME_STRING_CHECKED(UPhysicalMaterialMask, AssetImportData));
	FPropertyChangedEvent AssetImportDataChangedEvent(ParamProperty);
	PostEditChangeProperty(AssetImportDataChangedEvent);
}


void UPhysicalMaterialMask::DumpMaskData()
{
	int32 SizeX = 0;
	int32 SizeY = 0;
	TArray<uint32> MaskData;
	GenerateMaskData(MaskData, SizeX, SizeY);

	ensure(MaskData.Num() == SizeY * SizeX / 8);

	if (MaskData.Num() == 0)
	{
		UE_LOGF(LogPhysicalMaterialMask, Display, "PhysicalMaterialMask data is empty.");
	}

	auto GetColorString = [](uint32 ColorIdx) -> FString
	{
		switch (ColorIdx)
		{
		case EPhysicalMaterialMaskColor::White:
			return FString("White");
		case EPhysicalMaterialMaskColor::Black:
			return FString("Black");
		case EPhysicalMaterialMaskColor::Red:
			return FString("Red");
		case EPhysicalMaterialMaskColor::Green:
			return FString("Green");
		case EPhysicalMaterialMaskColor::Blue:
			return FString("Blue");
		case EPhysicalMaterialMaskColor::Yellow:
			return FString("Yellow");
		case EPhysicalMaterialMaskColor::Cyan:
			return FString("Cyan");
		case EPhysicalMaterialMaskColor::Magenta:
			return FString("Magenta");
		}
		return FString("INVALID_MASK_INDEX");
	};

	for (int32 Y = 0, ArrayIdx = 0; Y < SizeY; Y++)
	{
		UE_LOGF(LogPhysicalMaterialMask, Display, "Mask ROW %d:", Y);

		for (int32 X = 0; X < SizeX; ArrayIdx++)
		{
			// Unpack 8 color ids from one uint32.
			uint32 MaskEntry = MaskData[ArrayIdx];
			UE_LOGF(LogPhysicalMaterialMask, Display, "  Mask PackedEntry[%d][%d] = 0x%x", Y, X / 8, MaskEntry);

			for (int32 Idx = 0; Idx < 8 && X < SizeX; X++, Idx++)
			{
				uint32 ColorIdx = (MaskEntry >> (Idx * 4)) & 0xf;
				UE_LOGF(LogPhysicalMaterialMask, Display, "    Mask[%d][%d] = %ls (%d)", Y, X, *GetColorString(ColorIdx), ColorIdx);
			}
		}
	}
}

#endif // WITH_EDITOR

FPhysicsMaterialMaskHandle& UPhysicalMaterialMask::GetPhysicsMaterialMask()
{
	if(!MaterialMaskHandle)
	{
		//need to use ptr to avoid polluting engine headers
		MaterialMaskHandle = MakeUnique<FPhysicsMaterialMaskHandle>();
	}

	if (!MaterialMaskHandle->IsValid())
	{
		*MaterialMaskHandle = FPhysicsInterface::CreateMaterialMask(this);
		check(MaterialMaskHandle->IsValid());

		FPhysicsInterface::UpdateMaterialMask(*MaterialMaskHandle, this);
	}

	return *MaterialMaskHandle;
}

// This template generates mask data from the texture mask, converting colors to mask ids.
template<typename PixelDataType, int32 RIdx, int32 GIdx, int32 BIdx, int32 AIdx> class MaskDataGenerator
{
public:

	MaskDataGenerator(int32 SizeX, int32 SizeY, const uint8* SourceTextureData)
		: SourceData(reinterpret_cast<const PixelDataType*>(SourceTextureData))
		, TextureWidth(SizeX)
		, TextureHeight(SizeY)
	{
	}

	void GenerateMask(TArray<uint32>& OutMaskData)
	{
		check(TextureWidth % 8 == 0);

		OutMaskData.Empty();

		const PixelDataType* PixelData = SourceData;

		for (int32 Y = 0; Y < TextureHeight; ++Y)
		{
			for (int32 X = 0; X < TextureWidth; )
			{
				// Pack 8 color ids into one uint32.
				uint32 NewMaskEntry = 0x0;
				for (int32 Idx = 0; Idx < 8 && X < TextureWidth; X++, Idx++, PixelData += 4)
				{
					NewMaskEntry |= GetColorId(PixelData[RIdx], PixelData[GIdx], PixelData[BIdx]) << (Idx * 4);
				}
				OutMaskData.Emplace(NewMaskEntry);
			}
		}
	}

	bool IsEqual(FColor ColorA, FColor ColorB)
	{
		static int32 Tolerance = 10;
		return FMath::Abs(ColorA.R - ColorB.R) < Tolerance && FMath::Abs(ColorA.G - ColorB.G) < Tolerance && FMath::Abs(ColorA.B - ColorB.B) < Tolerance && FMath::Abs(ColorA.A - ColorB.A) < Tolerance;
	}

	uint32 GetColorId(PixelDataType R, PixelDataType G, PixelDataType B)
	{
		FColor Color(R, G, B);
		if (IsEqual(Color, FColor::White))
		{
			return EPhysicalMaterialMaskColor::White;
		}
		if (IsEqual(Color, FColor::Black))
		{
			return EPhysicalMaterialMaskColor::Black;
		}
		if (IsEqual(Color, FColor::Red))
		{
			return EPhysicalMaterialMaskColor::Red;
		}
		if (IsEqual(Color, FColor::Green))
		{
			return EPhysicalMaterialMaskColor::Green;
		}
		if (IsEqual(Color, FColor::Blue))
		{
			return EPhysicalMaterialMaskColor::Blue;
		}
		if (IsEqual(Color, FColor::Cyan))
		{
			return EPhysicalMaterialMaskColor::Cyan;
		}
		if (IsEqual(Color, FColor::Yellow))
		{
			return EPhysicalMaterialMaskColor::Yellow;
		}
		if (IsEqual(Color, FColor::Magenta))
		{
			return EPhysicalMaterialMaskColor::Magenta;
		}
		return UPhysicalMaterialMask::INVALID_MASK_INDEX;
	}

	const PixelDataType* SourceData;
	int32 TextureWidth;
	int32 TextureHeight;
};


void UPhysicalMaterialMask::GenerateMaskData(TArray<uint32>& OutMaskData, int32& OutSizeX, int32& OutSizeY) const
{
	OutMaskData.Empty();

#if WITH_EDITOR

	if (MaskTexture)
	{
		const uint8* TextureData = MaskTexture->Source.LockMipReadOnly(0);
		if (TextureData)
		{
			const int32 TextureDataSize = MaskTexture->Source.CalcMipSize(0);

			if (TextureDataSize > 0)
			{
				OutSizeX = MaskTexture->Source.GetSizeX();
				OutSizeY = MaskTexture->Source.GetSizeY();

				ETextureSourceFormat TextureDataSourceFormat = MaskTexture->Source.GetFormat();
				switch (TextureDataSourceFormat)
				{
				case TSF_BGRA8:
				{
					MaskDataGenerator<uint8, 2, 1, 0, 3> MaskDataGen(OutSizeX, OutSizeY, TextureData);
					MaskDataGen.GenerateMask(OutMaskData);
					break;
				}

				case TSF_RGBA16:
				{
					MaskDataGenerator<uint16, 0, 1, 2, 3> MaskDataGen(OutSizeX, OutSizeY, TextureData);
					MaskDataGen.GenerateMask(OutMaskData);
					break;
				}

				default:
					check(0);
					break;
				}
			}
		}

		MaskTexture->Source.UnlockMip(0);
	}
#endif // WITH_EDITOR

	if (OutMaskData.Num() == 0)
	{
		OutSizeX = 0;
		OutSizeY = 0;
	}
}

uint32 UPhysicalMaterialMask::GetPhysMatIndex(const TArray<uint32>& MaskData, int32 SizeX, int32 SizeY, int32 AddressX, int32 AddressY, float U, float V)
{
	auto AdjustCoord = [](float& Val, TextureAddress Address)
	{
		if (Address == TextureAddress::TA_Clamp)
		{
			Val = FMath::Clamp(Val, 0.0f, 1.0f);
		}
		else if (Address == TextureAddress::TA_Wrap)
		{
			Val = FMath::Frac(Val);
		}
		else // Mirror
		{
			float IntPart;
			Val = FMath::Modf(Val, &IntPart);
			if (static_cast<int32>(IntPart) % 2 > 0)
			{
				Val = 1.0f - Val;
			}
		}
	};

	AdjustCoord(U, static_cast<TextureAddress>(AddressX));
	AdjustCoord(V, static_cast<TextureAddress>(AddressY));

	uint32 X = FMath::FloorToInt((float)SizeX * U);
	uint32 Y = FMath::FloorToInt((float)SizeY * V);
	uint32 Index = (SizeX / 8) * Y + X / 8;

	if (Index < (uint32)MaskData.Num())
	{
		return (MaskData[Index] >> (X % 8 * 4)) & 0xf;
	}

	return UPhysicalMaterialMask::INVALID_MASK_INDEX;
}



