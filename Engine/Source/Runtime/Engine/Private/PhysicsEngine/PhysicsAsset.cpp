// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	PhysicsAsset.cpp
=============================================================================*/ 

#include "PhysicsEngine/PhysicsAsset.h"
#include "Animation/MirrorDataTable.h"
#include "Engine/SkeletalMesh.h"
#include "Engine/SkinnedAsset.h"
#include "UObject/FrameworkObjectVersion.h"
#include "Serialization/ObjectWriter.h"
#include "Serialization/ObjectReader.h"
#include "Components/SkeletalMeshComponent.h"
#include "PhysicsEngine/PhysicsConstraintTemplate.h"
#include "PhysicsEngine/SkeletalBodySetup.h"
#include "UObject/AssetRegistryTagsContext.h"
#include "UObject/ReleaseObjectVersion.h"
#include "UObject/UObjectIterator.h"
#include "UObject/FortniteSeasonBranchObjectVersion.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PhysicsAsset)

#if WITH_EDITOR
#include "Misc/MessageDialog.h"
#endif

#define LOCTEXT_NAMESPACE "PhysicsAsset"

bool bUseRBANForDefaultPhysicsAssetSolverType = false;
FAutoConsoleVariableRef CVarUseRBANForDefaultPhysicsAssetSolverType(TEXT("p.Chaos.UseRBANForDefaultPhysicsAssetSolverType"), bUseRBANForDefaultPhysicsAssetSolverType, TEXT("Boolean to use RBAN for default physics asset solver type (false by default)"));

FPhysicsAssetSolverSettings::FPhysicsAssetSolverSettings()
	: PositionIterations(6)
	, VelocityIterations(1)
	, ProjectionIterations(1)
	, CullDistance(3.0f)
	, MaxDepenetrationVelocity(0.0f)
	, FixedTimeStep(0.0f)
	, bUseLinearJointSolver(true)
	, bUseManifolds(true)
{
}

FSolverIterations::FSolverIterations()
	: SolverIterations(3)
	, JointIterations(2)
	, CollisionIterations(2)
	, SolverPushOutIterations(1)
	, JointPushOutIterations(1)
	, CollisionPushOutIterations(0)
{
}

///////////////////////////////////////	
//////////// UPhysicsAsset ////////////
///////////////////////////////////////

UPhysicsAsset::UPhysicsAsset(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	if(bUseRBANForDefaultPhysicsAssetSolverType)
	{
		SolverType = EPhysicsAssetSolverType::RBAN;
	}
	else
	{
		SolverType = EPhysicsAssetSolverType::World;
	}
}

void UPhysicsAsset::UpdateBoundsBodiesArray()
{
	BoundsBodies.Empty();

	for(int32 i=0; i<SkeletalBodySetups.Num(); i++)
	{
		if(ensure(SkeletalBodySetups[i]) && SkeletalBodySetups[i]->bConsiderForBounds)
		{
			BoundsBodies.Add(i);
		}
	}
}

void UPhysicsAsset::UpdateBodySetupIndexMap()
{
	// update BodySetupIndexMap
	BodySetupIndexMap.Empty();

	for(int32 i=0; i<SkeletalBodySetups.Num(); i++)
	{
		if (ensure(SkeletalBodySetups[i]))
		{
			BodySetupIndexMap.Add(SkeletalBodySetups[i]->BoneName, i);
		}
	}
}

void UPhysicsAsset::PostLoad()
{
	Super::PostLoad();

	if(GetLinkerCustomVersion(FFrameworkObjectVersion::GUID) < FFrameworkObjectVersion::PhysAssetUseSkeletalBodySetup)
	{
		SkeletalBodySetups.AddUninitialized(BodySetup_DEPRECATED.Num());
		for(int32 Idx = 0; Idx < BodySetup_DEPRECATED.Num(); ++Idx)
		{
			//Used to use BodySetup for physics asset, but moved to more specialized SkeletalBodySetup
			SkeletalBodySetups[Idx] = NewObject<USkeletalBodySetup>(this, NAME_None);
			
			TArray<uint8> OldData;
			FObjectWriter ObjWriter(BodySetup_DEPRECATED[Idx], OldData);
			FObjectReader ObjReader(SkeletalBodySetups[Idx], OldData);
		}
		
		BodySetup_DEPRECATED.Empty();
	}

	// Ensure array of bounds bodies is up to date.
	if(SkeletalBodySetups.Num() == 0)
	{
		UpdateBoundsBodiesArray();
	}

	if (SkeletalBodySetups.Num() > 0 && BodySetupIndexMap.Num() == 0)
	{
		UpdateBodySetupIndexMap();
	}
}

#if WITH_EDITORONLY_DATA
void UPhysicsAsset::DeclareConstructClasses(TArray<FTopLevelAssetPath>& OutConstructClasses, const UClass* SpecificSubclass)
{
	Super::DeclareConstructClasses(OutConstructClasses, SpecificSubclass);

	OutConstructClasses.Add(FTopLevelAssetPath(USkeletalBodySetup::StaticClass()));
}
#endif


void UPhysicsAsset::Serialize(FArchive& Ar)
{
	Ar.UsingCustomVersion(FFortniteSeasonBranchObjectVersion::GUID);
	Ar.UsingCustomVersion(FFortniteMainBranchObjectVersion::GUID);

	Super::Serialize(Ar);
	Ar << CollisionDisableTable;

#if WITH_EDITORONLY_DATA
	if (DefaultSkelMesh_DEPRECATED != NULL)
	{
		PreviewSkeletalMesh = TSoftObjectPtr<USkeletalMesh>(DefaultSkelMesh_DEPRECATED);
		DefaultSkelMesh_DEPRECATED = NULL;
	}
#endif

	// Transfer the legacy solver iteration counts to the new ones. These settings are used by the RBAN solver.
	// These settings are intended to be roughly equivalent to the previous settings (the new solver adds VelocityIterations
	// and provides the linear/non-linear joint solver option). Any new PhysicsAssets will get the defaults (new linear solver etc)
	const bool bRequiresSettingsTransfer = (Ar.CustomVer(FFortniteSeasonBranchObjectVersion::GUID) < FFortniteSeasonBranchObjectVersion::PhysicsAssetNewSolverSettings);
	if (bRequiresSettingsTransfer)
	{
		SolverSettings.PositionIterations = SolverIterations.SolverIterations * SolverIterations.JointIterations;
		SolverSettings.VelocityIterations = 1;
		SolverSettings.ProjectionIterations = SolverIterations.SolverPushOutIterations;
		SolverSettings.bUseLinearJointSolver = false;
		SolverSettings.CullDistance = 1.0f;
	}

	const bool bShouldNotUseManifoldOnOldAsset = (Ar.CustomVer(FFortniteMainBranchObjectVersion::GUID) < FFortniteMainBranchObjectVersion::PhysicsAssetUseManifoldFlags);
	if (bShouldNotUseManifoldOnOldAsset)
	{
		SolverSettings.bUseManifolds = false;
	}

	Ar.UsingCustomVersion(FFrameworkObjectVersion::GUID);
	Ar.UsingCustomVersion(FReleaseObjectVersion::GUID);
}


void UPhysicsAsset::EnableCollision(int32 BodyIndexA, int32 BodyIndexB)
{
	if(BodyIndexA == BodyIndexB)
	{
		return;
	}

	FRigidBodyIndexPair Key(BodyIndexA, BodyIndexB);

	// If its not in table - do nothing
	if( !CollisionDisableTable.Find(Key) )
	{
		return;
	}

	CollisionDisableTable.Remove(Key);
}

void UPhysicsAsset::DisableCollision(int32 BodyIndexA, int32 BodyIndexB)
{
	if(BodyIndexA == BodyIndexB)
	{
		return;
	}

	FRigidBodyIndexPair Key(BodyIndexA, BodyIndexB);

	// If its already in the disable table - do nothing
	if( CollisionDisableTable.Find(Key) )
	{
		return;
	}

	CollisionDisableTable.Add(Key, 0);
}

bool UPhysicsAsset::IsCollisionEnabled(int32 BodyIndexA, int32 BodyIndexB) const
{
	if(BodyIndexA == BodyIndexB)
	{
		return false;
	}

	if(CollisionDisableTable.Find(FRigidBodyIndexPair(BodyIndexA, BodyIndexB)))
	{
		return false;
	}

	return true;
}

void UPhysicsAsset::SetPrimitiveCollision(int32 BodyIndex, EAggCollisionShape::Type PrimitiveType, int32 PrimitiveIndex, ECollisionEnabled::Type CollisionEnabled)
{
	check(SkeletalBodySetups.IsValidIndex(BodyIndex));
	FKAggregateGeom* AggGeom = &SkeletalBodySetups[BodyIndex]->AggGeom;
	ensure(PrimitiveIndex < AggGeom->GetElementCount());
	AggGeom->GetElement(PrimitiveType, PrimitiveIndex)->SetCollisionEnabled(CollisionEnabled);
}

ECollisionEnabled::Type UPhysicsAsset::GetPrimitiveCollision(int32 BodyIndex, EAggCollisionShape::Type PrimitiveType, int32 PrimitiveIndex) const
{
	check(SkeletalBodySetups.IsValidIndex(BodyIndex));
	FKAggregateGeom* AggGeom = &SkeletalBodySetups[BodyIndex]->AggGeom;
	ensure(PrimitiveIndex < AggGeom->GetElementCount());
	return AggGeom->GetElement(PrimitiveType, PrimitiveIndex)->GetCollisionEnabled();
}

void UPhysicsAsset::SetPrimitiveContributeToMass(int32 BodyIndex, EAggCollisionShape::Type PrimitiveType, int32 PrimitiveIndex, bool bContributeToMass)
{
	check(SkeletalBodySetups.IsValidIndex(BodyIndex));
	FKAggregateGeom* AggGeom = &SkeletalBodySetups[BodyIndex]->AggGeom;
	ensure(PrimitiveIndex < AggGeom->GetElementCount());
	return AggGeom->GetElement(PrimitiveType, PrimitiveIndex)->SetContributeToMass(bContributeToMass);
}

bool UPhysicsAsset::GetPrimitiveContributeToMass(int32 BodyIndex, EAggCollisionShape::Type PrimitiveType, int32 PrimitiveIndex) const
{
	check(SkeletalBodySetups.IsValidIndex(BodyIndex));
	FKAggregateGeom* AggGeom = &SkeletalBodySetups[BodyIndex]->AggGeom;
	ensure(PrimitiveIndex < AggGeom->GetElementCount());
	return AggGeom->GetElement(PrimitiveType, PrimitiveIndex)->GetContributeToMass();
}

FBox UPhysicsAsset::CalcAABB(const USkinnedMeshComponent* MeshComp, const FTransform& LocalToWorld) const
{
	FBox Box(ForceInit);

	if (!MeshComp)
	{
		return Box;
	}

	FVector Scale3D = LocalToWorld.GetScale3D();
	if( Scale3D.IsUniform() )
	{
		const TArray<int32>* BodyIndexRefs = NULL;
		TArray<int32> AllBodies;
		// If we want to consider all bodies, make array with all body indices in
		if(MeshComp->bConsiderAllBodiesForBounds)
		{
			AllBodies.AddUninitialized(SkeletalBodySetups.Num());
			for(int32 i=0; i<SkeletalBodySetups.Num();i ++)
			{
				AllBodies[i] = i;
			}
			BodyIndexRefs = &AllBodies;
		}
		// Otherwise, use the cached shortlist of bodies to consider
		else
		{
			BodyIndexRefs = &BoundsBodies;
		}

		// Then iterate over bodies we want to consider, calculating bounding box for each
		const int32 BodySetupNum = (*BodyIndexRefs).Num();

		for(int32 i=0; i<BodySetupNum; i++)
		{
			const int32 BodyIndex = (*BodyIndexRefs)[i];
			UBodySetup* bs = SkeletalBodySetups[BodyIndex];

			// Check if setup should be considered for bounds, or if all bodies should be considered anyhow
			if (bs->bConsiderForBounds || MeshComp->bConsiderAllBodiesForBounds)
			{
				if (i+1<BodySetupNum)
				{
					int32 NextIndex = (*BodyIndexRefs)[i+1];
					FPlatformMisc::Prefetch(SkeletalBodySetups[NextIndex]);
					FPlatformMisc::Prefetch(SkeletalBodySetups[NextIndex], PLATFORM_CACHE_LINE_SIZE);
				}

				int32 BoneIndex = MeshComp->GetBoneIndex(bs->BoneName);
				if(BoneIndex != INDEX_NONE)
				{
					const FTransform WorldBoneTransform = MeshComp->GetBoneTransform(BoneIndex, LocalToWorld);
					FBox BodySetupBounds = bs->AggGeom.CalcAABB(WorldBoneTransform);

					// When the transform contains a negative scale CalcAABB could return a invalid FBox that has Min and Max reversed
					// @TODO: Maybe CalcAABB should handle that inside and never return a reversed FBox
					if (BodySetupBounds.Min.X > BodySetupBounds.Max.X)
					{
						Swap(BodySetupBounds.Min.X, BodySetupBounds.Max.X);
					}

					if (BodySetupBounds.Min.Y > BodySetupBounds.Max.Y)
					{
						Swap(BodySetupBounds.Min.Y, BodySetupBounds.Max.Y);
					}

					if (BodySetupBounds.Min.Z > BodySetupBounds.Max.Z)
					{
						Swap(BodySetupBounds.Min.Z, BodySetupBounds.Max.Z);
					}

					Box += BodySetupBounds;
				}
			}
		}
	}
	else
	{
		UE_LOGF(LogPhysics, Log,  "UPhysicsAsset::CalcAABB : Non-uniform scale factor. You will not be able to collide with it.  Turn off collision and wrap it with a blocking volume.  MeshComp: %ls  SkelMesh: %ls", *MeshComp->GetFullName(), MeshComp->GetSkinnedAsset() ? *MeshComp->GetSkinnedAsset()->GetFullName() : TEXT("NULL") );
	}

	if(!Box.IsValid)
	{
		Box = FBox( LocalToWorld.GetLocation(), LocalToWorld.GetLocation() );
	}

	const float MinBoundSize = 1.f;
	const FVector BoxSize = Box.GetSize();

	if(BoxSize.GetMin() < MinBoundSize)
	{
		const FVector ExpandByDelta ( FMath::Max<FVector::FReal>(0, MinBoundSize - BoxSize.X), FMath::Max<FVector::FReal>(0, MinBoundSize - BoxSize.Y), FMath::Max<FVector::FReal>(0, MinBoundSize - BoxSize.Z) );
		Box = Box.ExpandBy(ExpandByDelta * 0.5f);	//expand by applies to both directions with GetSize applies to total size so divide by 2
	}

	return Box;
}

#if WITH_EDITOR
bool UPhysicsAsset::CanCalculateValidAABB(const USkinnedMeshComponent* MeshComp, const FTransform& LocalToWorld) const
{
	bool ValidBox = false;

	if (!MeshComp)
	{
		return false;
	}

	FVector Scale3D = LocalToWorld.GetScale3D();
	if (!Scale3D.IsUniform())
	{
		return false;
	}

	for (int32 i = 0; i < SkeletalBodySetups.Num(); i++)
	{
		UBodySetup* bs = SkeletalBodySetups[i];
		// Check if setup should be considered for bounds, or if all bodies should be considered anyhow
		if (bs->bConsiderForBounds || MeshComp->bConsiderAllBodiesForBounds)
		{
			int32 BoneIndex = MeshComp->GetBoneIndex(bs->BoneName);
			if (BoneIndex != INDEX_NONE)
			{
				FTransform WorldBoneTransform = MeshComp->GetBoneTransform(BoneIndex, LocalToWorld);
				if (FMath::Abs(WorldBoneTransform.GetDeterminant()) >(float)UE_KINDA_SMALL_NUMBER)
				{
					FBox Box = bs->AggGeom.CalcAABB(WorldBoneTransform);
					if (Box.GetSize().SizeSquared() > (float)UE_KINDA_SMALL_NUMBER)
					{
						ValidBox = true;
						break;
					}
				}
			}
		}
	}
	return ValidBox;
}
#endif //WITH_EDITOR

int32 UPhysicsAsset::FindControllingBodyIndex(const class USkeletalMesh* skelMesh, int32 StartBoneIndex) const
{
	int32 BoneIndex = StartBoneIndex;
	while(BoneIndex!=INDEX_NONE)
	{
		FName BoneName = skelMesh->GetRefSkeleton().GetBoneName(BoneIndex);
		int32 BodyIndex = FindBodyIndex(BoneName);

		if(BodyIndex != INDEX_NONE)
			return BodyIndex;

		int32 ParentBoneIndex = skelMesh->GetRefSkeleton().GetParentIndex(BoneIndex);

		if(ParentBoneIndex == BoneIndex)
			return INDEX_NONE;

		BoneIndex = ParentBoneIndex;
	}

	return INDEX_NONE; // Shouldn't reach here.
}

int32 UPhysicsAsset::FindParentBodyIndex(const class USkeletalMesh* skelMesh, int32 StartBoneIndex) const
{
	if (skelMesh)
	{
		return FindParentBodyIndex(skelMesh->GetRefSkeleton(), StartBoneIndex);
	}

	return INDEX_NONE;
}

int32 UPhysicsAsset::FindParentBodyIndex(const FReferenceSkeleton& RefSkeleton, const int32 StartBoneIndex) const
{
	int32 BoneIndex = StartBoneIndex;
	while ((BoneIndex = RefSkeleton.GetParentIndex(BoneIndex)) != INDEX_NONE)
	{
		const FName BoneName = RefSkeleton.GetBoneName(BoneIndex);
		const int32 BodyIndex = FindBodyIndex(BoneName);

		if (StartBoneIndex == BoneIndex)
			return INDEX_NONE;

		if (BodyIndex != INDEX_NONE)
			return BodyIndex;
	}

	return INDEX_NONE;
}


int32 UPhysicsAsset::FindBodyIndex(FName bodyName) const
{
	const int32* IdxData = BodySetupIndexMap.Find(bodyName);
	if (IdxData)
	{
		return *IdxData;
	}

	return INDEX_NONE;
}

int32 UPhysicsAsset::FindConstraintIndex(FName ConstraintName) const
{
	for(int32 i=0; i<ConstraintSetup.Num(); i++)
	{
		if( ConstraintSetup[i]->DefaultInstance.JointName == ConstraintName )
		{
			return i;
		}
	}

	return INDEX_NONE;
}

int32 UPhysicsAsset::FindConstraintIndex(FName Bone1Name, FName Bone2Name) const
{
	for (int32 i = 0; i < ConstraintSetup.Num(); i++)
	{
		if (ConstraintSetup[i]->DefaultInstance.ConstraintBone1 == Bone1Name &&
			ConstraintSetup[i]->DefaultInstance.ConstraintBone2 == Bone2Name)
		{
			return i;
		}
	}

	return INDEX_NONE;
}

FName UPhysicsAsset::FindConstraintBoneName(int32 ConstraintIndex) const
{
	if ( (ConstraintIndex < 0) || (ConstraintIndex >= ConstraintSetup.Num()) )
	{
		return NAME_None;
	}

	return ConstraintSetup[ConstraintIndex]->DefaultInstance.GetChildBoneName();
}

int32 UPhysicsAsset::FindMirroredBone(const USkeletalMesh* SkelMesh,  int32 BoneIndex) const 
{
	if (SkelMesh)
	{
		FName BoneName = SkelMesh->GetRefSkeleton().GetBoneName(BoneIndex);
		FName MirrorName = UMirrorDataTable::GetSettingsMirrorName(BoneName);
		return SkelMesh->GetRefSkeleton().FindBoneIndex(MirrorName); 
	}

	return INDEX_NONE; 
}

void UPhysicsAsset::GetBodyIndicesBelow(TArray<int32>& OutBodyIndices, FName InBoneName, const USkeletalMesh* SkelMesh, bool bIncludeParent /*= true*/) const
{
	if (SkelMesh)
	{
		GetBodyIndicesBelow(OutBodyIndices, InBoneName, SkelMesh->GetRefSkeleton(), bIncludeParent);
	}
}

void UPhysicsAsset::GetBodyIndicesBelow(TArray<int32>& OutBodyIndices, const FName InBoneName, const FReferenceSkeleton& RefSkeleton, const bool bIncludeParent /*= true*/) const
{
	const int32 BaseIndex = RefSkeleton.FindBoneIndex(InBoneName);

	if (BaseIndex != INDEX_NONE)
	{
		// Iterate over all other bodies, looking for 'children' of this one
		for (int32 i = 0; i < SkeletalBodySetups.Num(); i++)
		{
			const UBodySetup* BS = SkeletalBodySetups[i];
			if (!ensure(BS))
			{
				continue;
			}
			FName TestName = BS->BoneName;
			int32 TestIndex = RefSkeleton.FindBoneIndex(TestName);

			if ((bIncludeParent && TestIndex == BaseIndex) || RefSkeleton.BoneIsChildOf(TestIndex, BaseIndex))
			{
				OutBodyIndices.Add(i);
			}
		}
	}
}

void UPhysicsAsset::GetNearestBodyIndicesBelow(TArray<int32> & OutBodyIndices, FName InBoneName, const USkeletalMesh* InSkelMesh) const
{
	TArray<int32> AllBodiesBelow;
	GetBodyIndicesBelow(AllBodiesBelow, InBoneName, InSkelMesh, false);

	//we need to filter all bodies below to first in the chain
	TArray<bool> Nearest;
	Nearest.AddUninitialized(SkeletalBodySetups.Num());
	for (int32 i = 0; i < Nearest.Num(); ++i)
	{
		Nearest[i] = true;
	}

	for (int32 i = 0; i < AllBodiesBelow.Num(); i++)
	{
		int32 BodyIndex = AllBodiesBelow[i];
		if (Nearest[BodyIndex] == false) continue;

		const UBodySetup * Body = SkeletalBodySetups[BodyIndex];
		if (!ensure(Body))
		{
			continue;
		}
		TArray<int32> BodiesBelowMe;
		GetBodyIndicesBelow(BodiesBelowMe, Body->BoneName, InSkelMesh, false);
		
		for (int j = 0; j < BodiesBelowMe.Num(); ++j)
		{
			Nearest[BodiesBelowMe[j]] = false;	
		}
	}

	for (int32 i = 0; i < AllBodiesBelow.Num(); i++)
	{
		int32 BodyIndex = AllBodiesBelow[i];
		if (Nearest[BodyIndex])
		{
			OutBodyIndices.Add(BodyIndex);
		}
	}
}

void UPhysicsAsset::ClearAllPhysicsMeshes()
{
	for(int32 i=0; i<SkeletalBodySetups.Num(); i++)
	{
		if (ensure(SkeletalBodySetups[i]))
		{
			SkeletalBodySetups[i]->ClearPhysicsMeshes();
		}
	}
}

#if WITH_EDITOR

void UPhysicsAsset::InvalidateAllPhysicsMeshes()
{
	for(int32 i=0; i<SkeletalBodySetups.Num(); i++)
	{
		if (ensure(SkeletalBodySetups[i]))
		{
			SkeletalBodySetups[i]->InvalidatePhysicsData();
		}
	}
}


void UPhysicsAsset::PostEditUndo()
{
	Super::PostEditUndo();
	
	UpdateBodySetupIndexMap();
	UpdateBoundsBodiesArray();
}

void UPhysicsAsset::PreEditChange(FProperty* PropertyThatWillChange)
{
	Super::PreEditChange(PropertyThatWillChange);

	PreConstraintProfiles = ConstraintProfiles;
	PrePhysicalAnimationProfiles = PhysicalAnimationProfiles;
}

template <typename T>
void SanitizeProfilesHelper(const TArray<T*>& SetupInstances, const TArray<FName>& PreProfiles, TArray<FName>& PostProfiles, FPropertyChangedEvent& PropertyChangedEvent, const FName PropertyName, FName& CurrentProfileName, TFunctionRef<void (T*, FName, FName)> RenameFunc, TFunctionRef<void(T*, FName, FName)> DuplicateFunc, TFunctionRef<void(T*, const TArray<FName>& )> UpdateFunc)
{
	const int32 ArrayIdx = PropertyChangedEvent.GetArrayIndex(PropertyName.ToString());
	const FName OldName = PreProfiles.IsValidIndex(ArrayIdx) ? PreProfiles[ArrayIdx] : NAME_None;

	if (ArrayIdx != INDEX_NONE)
	{
		if(PropertyChangedEvent.ChangeType != EPropertyChangeType::ArrayMove && PropertyChangedEvent.ChangeType != EPropertyChangeType::ArrayRemove)
		{
			int32 CollisionCount = 0;
			FName NewName = (PostProfiles[ArrayIdx] == NAME_None) ? FName("New") : PostProfiles[ArrayIdx];
			const FString NewNameNoNumber = NewName.ToString();
			while(PreProfiles.Contains(NewName))
			{
				FString NewNameStr = NewNameNoNumber;
				NewNameStr.Append("_").AppendInt(++CollisionCount);
				NewName = FName(*NewNameStr);
			}

			PostProfiles[ArrayIdx] = NewName;
		}
	}
	

	if (PropertyChangedEvent.ChangeType == EPropertyChangeType::ValueSet || PropertyChangedEvent.ChangeType == EPropertyChangeType::ArrayClear)
	{
		if (ArrayIdx != INDEX_NONE)	//INDEX_NONE can come when emptying the array, so just ignore it
		{
			for (T* SetupInstance : SetupInstances)
			{
				if (!SetupInstance)
				{
					continue;
				}
				SetupInstance->Modify();
				RenameFunc(SetupInstance, PreProfiles[ArrayIdx], PostProfiles[ArrayIdx]);
			}

			if (CurrentProfileName == PreProfiles[ArrayIdx])
			{
				CurrentProfileName = PostProfiles[ArrayIdx];
			}
		}
	}

	if (PropertyChangedEvent.ChangeType == EPropertyChangeType::Duplicate)
	{
		for (T* SetupInstance : SetupInstances)
		{
			if (!SetupInstance)
			{
				continue;
			}
			SetupInstance->Modify();
			DuplicateFunc(SetupInstance, OldName, PostProfiles[ArrayIdx]);
		}
	}

	//array events like empty currently do not get an Empty type so we need to do this final sanitization every time something changes just in case
	{
		//delete requires removing old profiles
		for (T* SetupInstance : SetupInstances)
		{
			if (!SetupInstance)
			{
				continue;
			}
			SetupInstance->Modify();
			UpdateFunc(SetupInstance, PostProfiles);
		}

		if (PostProfiles.Find(CurrentProfileName) == INDEX_NONE)
		{
			CurrentProfileName = NAME_None;
		}
	}
}

void UPhysicsAsset::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	if(FProperty* Property = PropertyChangedEvent.Property)
	{
		const FName PropertyName = Property->GetFName();
		if(PropertyName == GET_MEMBER_NAME_CHECKED(UPhysicsAsset, PhysicalAnimationProfiles))
		{
			auto RenameFunc = [](USkeletalBodySetup* BS, FName PreName, FName NewName)
			{
				BS->RenamePhysicalAnimationProfile(PreName, NewName);
			};

			auto DuplicateFunc = [](USkeletalBodySetup* BS, FName DuplicateFromName, FName DuplicateToName)
			{
				BS->DuplicatePhysicalAnimationProfile(DuplicateFromName, DuplicateToName);
			};

			auto UpdateFunc = [](USkeletalBodySetup* BS, const TArray<FName>& NewProfiles)
			{
				BS->UpdatePhysicalAnimationProfiles(NewProfiles);
			};

			SanitizeProfilesHelper<USkeletalBodySetup>(SkeletalBodySetups, PrePhysicalAnimationProfiles, PhysicalAnimationProfiles, PropertyChangedEvent, PropertyName, CurrentPhysicalAnimationProfileName, RenameFunc, DuplicateFunc, UpdateFunc);
		}
		else if (PropertyName == GET_MEMBER_NAME_CHECKED(UPhysicsAsset, ConstraintProfiles))
		{
			auto RenameFunc = [](UPhysicsConstraintTemplate* CS, FName PreName, FName NewName)
			{
				CS->RenameConstraintProfile(PreName, NewName);
			};

			auto DuplicateFunc = [](UPhysicsConstraintTemplate* CS, FName DuplicateFromName, FName DuplicateToName)
			{
				CS->DuplicateConstraintProfile(DuplicateFromName, DuplicateToName);
			};

			auto UpdateFunc = [](UPhysicsConstraintTemplate* CS, const TArray<FName>& NewProfiles)
			{
				CS->UpdateConstraintProfiles(NewProfiles);
			};

			SanitizeProfilesHelper<UPhysicsConstraintTemplate>(ConstraintSetup, PreConstraintProfiles, ConstraintProfiles, PropertyChangedEvent, PropertyName, CurrentConstraintProfileName, RenameFunc, DuplicateFunc, UpdateFunc);
		}
	}

	for(UAssetUserData* UserData : AssetUserData)
	{
		if(UserData)
		{
			UserData->PostEditChangeOwner(PropertyChangedEvent);
		}
	}

	Super::PostEditChangeProperty(PropertyChangedEvent);

	RefreshPhysicsAssetChange();
}

EDataValidationResult UPhysicsAsset::IsDataValid(FDataValidationContext& Context) const
{
	EDataValidationResult Result = EDataValidationResult::Valid;
	for (USkeletalBodySetup* BodySetup : SkeletalBodySetups)
	{
		Result = CombineDataValidationResults(Result, BodySetup->IsDataValid(Context));
	}
	return Result;
}

#endif

//// THUMBNAIL SUPPORT //////

/** 
 * Returns a one line description of an object for viewing in the thumbnail view of the generic browser
 */
FString UPhysicsAsset::GetDesc()
{
	return FString::Printf( TEXT("%d Bodies, %d Constraints"), SkeletalBodySetups.Num(), ConstraintSetup.Num() );
}

void UPhysicsAsset::GetAssetRegistryTags(TArray<FAssetRegistryTag>& OutTags) const
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS;
	Super::GetAssetRegistryTags(OutTags);
	PRAGMA_ENABLE_DEPRECATION_WARNINGS;
}

void UPhysicsAsset::GetAssetRegistryTags(FAssetRegistryTagsContext Context) const
{
	Context.AddTag( FAssetRegistryTag("Bodies", FString::FromInt(SkeletalBodySetups.Num()), FAssetRegistryTag::TT_Numerical) );
	Context.AddTag( FAssetRegistryTag("Constraints", FString::FromInt(ConstraintSetup.Num()), FAssetRegistryTag::TT_Numerical) );

	Super::GetAssetRegistryTags(Context);
}

#if WITH_EDITOR
void UPhysicsAsset::GetAssetRegistryTagMetadata(TMap<FName, FAssetRegistryTagMetadata>& OutMetadata) const
{
	Super::GetAssetRegistryTagMetadata(OutMetadata);

	OutMetadata.Add("Bodies",
		FAssetRegistryTagMetadata()
			.SetDisplayName(NSLOCTEXT("UPhysicsAsset", "Bodies_Label", "Bodies"))
		);

	OutMetadata.Add("Constraints",
		FAssetRegistryTagMetadata()
			.SetDisplayName(NSLOCTEXT("UPhysicsAsset", "Constraints_Label", "Constraints"))
		);
}
#endif


void UPhysicsAsset::BodyFindConstraints(int32 BodyIndex, TArray<int32>& Constraints)
{
	Constraints.Empty();
	FName BodyName = SkeletalBodySetups[BodyIndex]->BoneName;

	for(int32 ConIdx=0; ConIdx<ConstraintSetup.Num(); ConIdx++)
	{
		if( ConstraintSetup[ConIdx]->DefaultInstance.ConstraintBone1 == BodyName || ConstraintSetup[ConIdx]->DefaultInstance.ConstraintBone2 == BodyName )
		{
			Constraints.Add(ConIdx);
		}
	}
}

#if WITH_EDITOR
UPhysicsAsset::FRefreshPhysicsAssetChangeDelegate UPhysicsAsset::OnRefreshPhysicsAssetChange;

void UPhysicsAsset::RefreshPhysicsAssetChange() const
{
	for (FThreadSafeObjectIterator Iter(USkeletalMeshComponent::StaticClass()); Iter; ++Iter)
	{
		USkeletalMeshComponent* SkeletalMeshComponent = Cast<USkeletalMeshComponent>(*Iter);
		if (SkeletalMeshComponent->GetPhysicsAsset() == this)
		{
			// it needs to recreate IF it already has been created
			if (SkeletalMeshComponent->IsPhysicsStateCreated() && SkeletalMeshComponent->Bodies.Num() > 0)
			{
				SkeletalMeshComponent->RecreatePhysicsState();
				SkeletalMeshComponent->InvalidateCachedBounds();
				SkeletalMeshComponent->UpdateBounds();
				SkeletalMeshComponent->MarkRenderTransformDirty();
			}
		}
	}

	OnRefreshPhysicsAssetChange.Broadcast(this);
}

#endif

USkeletalMesh* UPhysicsAsset::GetPreviewMesh() const
{
#if WITH_EDITORONLY_DATA
	USkeletalMesh* PreviewMesh = PreviewSkeletalMesh.Get();
	if(!PreviewMesh)
	{
		// if preview mesh isn't loaded, see if we have set
		FSoftObjectPath PreviewMeshStringRef = PreviewSkeletalMesh.ToSoftObjectPath();
		// load it since now is the time to load
		if(!PreviewMeshStringRef.ToString().IsEmpty())
		{
			PreviewMesh = Cast<USkeletalMesh>(StaticLoadObject(USkeletalMesh::StaticClass(), nullptr, *PreviewMeshStringRef.ToString(), nullptr, LOAD_None, nullptr));
		}
	}

	return PreviewMesh;
#else
	return nullptr;
#endif
}

#if WITH_EDITOR
FConstraintInstanceAccessor UPhysicsAsset::GetConstraintInstanceAccessorByIndex(int32 ConstraintIndex)
{
	if (ConstraintIndex == INDEX_NONE || ConstraintIndex >= ConstraintSetup.Num())
	{
		return FConstraintInstanceAccessor();
	}

	if (ConstraintSetup[ConstraintIndex])
	{
		// Implementation note: Any changes on the constraint must be propagated from the instance to its profile
		return FConstraintInstanceAccessor(this, ConstraintIndex, [this, ConstraintIndex]() {
			ConstraintSetup[ConstraintIndex]->UpdateProfileInstance(); });
	}

	return FConstraintInstanceAccessor();
}

FConstraintInstanceAccessor UPhysicsAsset::GetConstraintByName(FName ConstraintName)
{
	return GetConstraintInstanceAccessorByIndex(FindConstraintIndex(ConstraintName));
}

FConstraintInstanceAccessor UPhysicsAsset::GetConstraintByBoneNames(FName Bone1Name, FName Bone2Name)
{
	return GetConstraintInstanceAccessorByIndex(FindConstraintIndex(Bone1Name, Bone2Name));
}

void UPhysicsAsset::GetConstraints(bool bIncludesTerminated, TArray<FConstraintInstanceAccessor>& OutConstraints)
{
	for (int32 i = 0; i < ConstraintSetup.Num(); ++i)
	{
		if (ConstraintSetup[i])
		{
			if (bIncludesTerminated || ConstraintSetup[i]->DefaultInstance.IsTerminated())
			{
				// Implementation note: Any changes on the constraint must be propagated from the instance to its profile
				OutConstraints.Emplace(this, i, [this, i]() {
					ConstraintSetup[i]->UpdateProfileInstance();
					});
			}
		}
	}
}

FConstraintInstance* UPhysicsAsset::GetConstraintInstanceByIndex(uint32 Index)
{
	if (Index < (uint32)ConstraintSetup.Num())
	{
		if (ConstraintSetup[Index])
		{
			return &ConstraintSetup[Index]->DefaultInstance;
		}
	}

	return nullptr;
}
#endif

void UPhysicsAsset::SetPreviewMesh(USkeletalMesh* PreviewMesh, bool bMarkAsDirty/*=true*/)
{
#if WITH_EDITORONLY_DATA
	if(PreviewMesh)
	{
		// See if any bones are missing from the skeletal mesh we are trying to use
		// @todo Could do more here - check for bone lengths etc. Maybe modify asset?
		for (int32 i = 0; i < SkeletalBodySetups.Num(); ++i)
		{
			FName BodyName = SkeletalBodySetups[i]->BoneName;
			int32 BoneIndex = PreviewMesh->GetRefSkeleton().FindBoneIndex(BodyName);
			if (BoneIndex == INDEX_NONE)
			{
				FMessageDialog::Open(EAppMsgType::Ok,
					FText::Format( LOCTEXT("BoneMissingFromSkelMesh", "The SkeletalMesh is missing bone '{0}' needed by this PhysicsAsset."), FText::FromName(BodyName) ));
				return;
			}
		}
	}

	if(bMarkAsDirty)
	{
		Modify();
	}
	PreviewSkeletalMesh = PreviewMesh;
#endif
}

void UPhysicsAsset::GetResourceSizeEx(FResourceSizeEx& CumulativeResourceSize)
{
	Super::GetResourceSizeEx(CumulativeResourceSize);

	// Nested body setups are handled by default implementation
	CumulativeResourceSize.AddDedicatedSystemMemoryBytes(BodySetupIndexMap.GetAllocatedSize());
	CumulativeResourceSize.AddDedicatedSystemMemoryBytes(CollisionDisableTable.GetAllocatedSize());
}

void UPhysicsAsset::AddAssetUserData(UAssetUserData* InUserData)
{
	if(!InUserData)
	{
		return;
	}

	UAssetUserData* ExistingData = GetAssetUserDataOfClass(InUserData->GetClass());
	if(ExistingData)
	{
		AssetUserData.Remove(ExistingData);
	}
	AssetUserData.Add(InUserData);
}

UAssetUserData* UPhysicsAsset::GetAssetUserDataOfClass(TSubclassOf<UAssetUserData> InUserDataClass)
{
	TObjectPtr<UAssetUserData>* Found = AssetUserData.FindByPredicate([&InUserDataClass](TObjectPtr<UAssetUserData> Item)
		{
			return Item && Item->IsA(InUserDataClass);
		});

	if(Found)
	{
		return *Found;
	}
	return nullptr;
}

void UPhysicsAsset::RemoveUserDataOfClass(TSubclassOf<UAssetUserData> InUserDataClass)
{
	AssetUserData.RemoveAll([&InUserDataClass](TObjectPtr<UAssetUserData> Item)
	{
		return Item && Item->IsA(InUserDataClass);
	});
}

const TArray<UAssetUserData*>* UPhysicsAsset::GetAssetUserDataArray() const
{
	return &ToRawPtrTArrayUnsafe(AssetUserData);
}

#undef LOCTEXT_NAMESPACE

