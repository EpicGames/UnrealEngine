// Copyright Epic Games, Inc. All Rights Reserved.

#include "PhysicsEngine/PhysicsBodyInstanceOwnerInterface.h"
#include "Engine/HitResult.h"
#include "Engine/OverlapResult.h"
#include "Components/PrimitiveComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PhysicsBodyInstanceOwnerInterface)

const IPhysicsBodyInstanceOwner* IPhysicsBodyInstanceOwner::GetPhysicsBodyInstanceOwnerFromHitResult(const FHitResult& Result, bool bPreferComponent)
{
	if (bPreferComponent)
	{
		if (const UPrimitiveComponent* Component = Result.GetComponent())
		{
			return Component;
		}
	}

	if (IPhysicsBodyInstanceOwnerResolver* PhysicsObjectOwnerResolver = Result.PhysicsObject ? Cast<IPhysicsBodyInstanceOwnerResolver>(Result.PhysicsObjectOwner.Get()) : nullptr)
	{
		if (IPhysicsBodyInstanceOwner* BodyInstanceOwner = PhysicsObjectOwnerResolver->ResolvePhysicsBodyInstanceOwner(Result.PhysicsObject))
		{
			return BodyInstanceOwner;
		}
	}

	return !bPreferComponent ? Result.GetComponent() : nullptr;
}

const IPhysicsBodyInstanceOwner* IPhysicsBodyInstanceOwner::GetPhysicsBodyInstanceOwnerFromOverlapResult(const FOverlapResult& Result, bool bPreferComponent)
{
	if (bPreferComponent)
	{
		if (const UPrimitiveComponent* Component = Result.GetComponent())
		{
			return Component;
		}
	}

	if (IPhysicsBodyInstanceOwnerResolver* PhysicsObjectOwnerResolver = Result.PhysicsObject ? Cast<IPhysicsBodyInstanceOwnerResolver>(Result.PhysicsObjectOwner.Get()) : nullptr)
	{
		if (IPhysicsBodyInstanceOwner* BodyInstanceOwner = PhysicsObjectOwnerResolver->ResolvePhysicsBodyInstanceOwner(Result.PhysicsObject))
		{
			return BodyInstanceOwner;
		}
	}

	return !bPreferComponent ? Result.GetComponent() : nullptr;
}
