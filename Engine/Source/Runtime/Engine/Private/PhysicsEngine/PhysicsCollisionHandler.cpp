// Copyright Epic Games, Inc. All Rights Reserved.

#include "PhysicsEngine/PhysicsCollisionHandler.h"
#include "ChaosLog.h"
#include "Kismet/GameplayStatics.h"
#include "Components/PrimitiveComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PhysicsCollisionHandler)

UPhysicsCollisionHandler::UPhysicsCollisionHandler(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	ImpactReFireDelay = 0.1f;
}

void UPhysicsCollisionHandler::DefaultHandleCollision_AssumesLocked(const FRigidBodyCollisionInfo& MyInfo, const FRigidBodyCollisionInfo& OtherInfo, const FCollisionImpactData& RigidCollisionData)
{
	FRigidBodyContactInfo ContactInfo = RigidCollisionData.ContactInfos[0];

	FBodyInstance* BodyInst0 = MyInfo.GetBodyInstance();
	FBodyInstance* BodyInst1 = OtherInfo.GetBodyInstance();

	if(BodyInst0 && BodyInst1)
	{
		const FPhysicsActorHandle Handle0 = BodyInst0->GetPhysicsActor();
		const FPhysicsActorHandle Handle1 = BodyInst1->GetPhysicsActor();

		if(!FPhysicsInterface::IsValid(Handle0) || !FPhysicsInterface::IsValid(Handle1))
		{
			const UPrimitiveComponent* Comp0 = BodyInst0->OwnerComponent.Get();
			const UPrimitiveComponent* Comp1 = BodyInst1->OwnerComponent.Get();

			if(Comp0 && Comp1)
			{
				UE_LOGF(LogChaos, Warning, "Collision handler encountered invalid actor\n\t%ls:%ls [%ls] - %ls\n\t%ls:%ls [%ls] - %ls",
					   *Comp0->GetOwner()->GetName(), *Comp0->GetName(), *Comp0->GetClass()->GetName(), FPhysicsInterface::IsValid(Handle0) ? TEXT("Valid") : TEXT("Invalid"),
					   *Comp1->GetOwner()->GetName(), *Comp1->GetName(), *Comp1->GetClass()->GetName(), FPhysicsInterface::IsValid(Handle1) ? TEXT("Valid") : TEXT("Invalid"));
			}
			else
			{
				UE_LOGF(LogChaos, Warning, "Collision handler encountered invalid actors and could not recover components");
			}

			return;
		}

		// Find relative velocity.
		FVector Velocity0 = BodyInst0->GetUnrealWorldVelocityAtPoint_AssumesLocked(ContactInfo.ContactPosition);
		FVector AngularVel0 = FMath::RadiansToDegrees(BodyInst0->GetUnrealWorldAngularVelocityInRadians_AssumesLocked());

		FVector Velocity1 = BodyInst1->GetUnrealWorldVelocityAtPoint_AssumesLocked(ContactInfo.ContactPosition);
		FVector AngularVel1 = FMath::RadiansToDegrees(BodyInst1->GetUnrealWorldAngularVelocityInRadians_AssumesLocked());

		const FVector RelVel = Velocity1 - Velocity0;

		// Then project along contact normal, and take magnitude.
		float ImpactVelMag = FMath::Abs(RelVel | ContactInfo.ContactNormal);

		// Difference in angular velocity between contacting bodies.
		float AngularVelMag = (AngularVel1 - AngularVel0).Size() * 70.f;

		// If bodies collide and are rotating quickly, even if relative linear velocity is not that high, use the value from the angular velocity instead.
		if (ImpactVelMag < AngularVelMag)
		{
			ImpactVelMag = AngularVelMag;
		}

		UWorld* World = GetWorld();

		if( (World != NULL) && (DefaultImpactSound != NULL) && (ImpactVelMag > ImpactThreshold) )
		{
			UGameplayStatics::PlaySoundAtLocation(World, DefaultImpactSound, ContactInfo.ContactPosition);

			LastImpactSoundTime = World->GetTimeSeconds();
		}
	}
}

void UPhysicsCollisionHandler::HandlePhysicsCollisions_AssumesLocked(TArray<FCollisionNotifyInfo>& PendingCollisionNotifies)
{
	// Fire any collision notifies in the queue.
	for(int32 i=0; i<PendingCollisionNotifies.Num(); i++)
	{
		// If it hasn't been long enough since our last sound, just bail out
		const float TimeSinceLastImpact = GetWorld()->GetTimeSeconds() - LastImpactSoundTime;
		if(TimeSinceLastImpact < ImpactReFireDelay)
		{
			break;
		}

		// See if this impact is between 2 valid actors
		const FCollisionNotifyInfo& NotifyInfo = PendingCollisionNotifies[i];
		if( NotifyInfo.IsValidForNotify() &&
			NotifyInfo.RigidCollisionData.ContactInfos.Num() > 0 )
		{
			DefaultHandleCollision_AssumesLocked(NotifyInfo.Info0, NotifyInfo.Info1, NotifyInfo.RigidCollisionData);
		}
	}
}

