// Copyright Epic Games, Inc. All Rights Reserved.

#include "PhysicsEngine/PhysicsConstraintActor.h"
#include "Async/TaskGraphInterfaces.h"
#include "Engine/Texture2D.h"
#include "UObject/ConstructorHelpers.h"
#include "PhysicsEngine/PhysicsConstraintComponent.h"
#include "PhysicsEngine/ConstraintUtils.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PhysicsConstraintActor)

APhysicsConstraintActor::APhysicsConstraintActor(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	// Structure to hold one-time initialization
	struct FConstructorStatics
	{
		ConstructorHelpers::FObjectFinderOptional<UTexture2D> KBSJointTexture;
		FName NAME_Physics;
		FConstructorStatics()
			: KBSJointTexture(TEXT("/Engine/EditorResources/S_KBSJoint"))
			, NAME_Physics(TEXT("Physics"))
		{
		}
	};
	static FConstructorStatics ConstructorStatics;

	ConstraintComp = CreateDefaultSubobject<UPhysicsConstraintComponent>(TEXT("MyConstraintComp"));
	RootComponent = ConstraintComp;
	SetHidden(true);
}

void APhysicsConstraintActor::PostLoad()
{
	Super::PostLoad();

	// Copy 'actors to constrain' into component
	if (GetLinkerUEVersion() < VER_UE4_ALL_PROPS_TO_CONSTRAINTINSTANCE && ConstraintComp != NULL)
	{
		ConstraintComp->ConstraintActor1 = ConstraintActor1_DEPRECATED;
		ConstraintComp->ConstraintActor2 = ConstraintActor2_DEPRECATED;
		ConstraintComp->ConstraintInstance.ProfileInstance.bDisableCollision = bDisableCollision_DEPRECATED;
	}
}

#if WITH_EDITOR
void APhysicsConstraintActor::LoadedFromAnotherClass( const FName& OldClassName )
{
	Super::LoadedFromAnotherClass(OldClassName);

	static const FName PhysicsBSJointActor_NAME("PhysicsBSJointActor");
	static const FName PhysicsHingeActor_NAME("PhysicsHingeActor");
	static const FName PhysicsPrismaticActor_NAME("PhysicsPrismaticActor");

	if (OldClassName == PhysicsHingeActor_NAME)
	{
		ConstraintUtils::ConfigureAsHinge(ConstraintComp->ConstraintInstance, false);
	}
	else if (OldClassName == PhysicsPrismaticActor_NAME)
	{
		ConstraintUtils::ConfigureAsPrismatic(ConstraintComp->ConstraintInstance, false);
	}
	else if (OldClassName == PhysicsBSJointActor_NAME)
	{
		ConstraintUtils::ConfigureAsBallAndSocket(ConstraintComp->ConstraintInstance, false);

	}

	ConstraintComp->UpdateSpriteTexture();
}
#endif // WITH_EDITOR


