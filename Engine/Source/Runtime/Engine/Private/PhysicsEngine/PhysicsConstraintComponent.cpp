// Copyright Epic Games, Inc. All Rights Reserved.

#include "PhysicsEngine/PhysicsConstraintComponent.h"
#include "Components/SkeletalMeshComponent.h"
#include "Engine/Texture2D.h"
#include "Engine/World.h"
#include "Logging/MessageLog.h"
#include "Misc/UObjectToken.h"
#include "PhysicsEngine/PhysicsConstraintTemplate.h"
#include "PhysicsEngine/PhysicsAsset.h"
#include "PhysicsEngine/ConstraintUtils.h"
#include "PhysicsEngine/PhysicsObjectExternalInterface.h"
#include "PhysicsEngine/SkeletalBodySetup.h"
#include "PhysicsProxy/JointConstraintProxy.h"
#include "Chaos/PBDJointConstraintData.h"
#include "Chaos/PhysicsObjectInterface.h"
#include "PBDRigidsSolver.h"
#include "PhysicsProxy/SingleParticlePhysicsProxy.h"
#include "Components/BillboardComponent.h"
#include "UObject/ICookInfo.h"
#include "UObject/SoftObjectPath.h"

#if UE_WITH_REMOTE_OBJECT_HANDLE
#include "UObject/UObjectMigrationContext.h"
#include "PhysicsInterfaceDeclaresCore.h"
#include "Physics/Experimental/PhysScene_Chaos.h"
#include "Chaos/Serialization/SerializedMultiPhysicsState.h"
#include "UObject/UObjectMigrationContext.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(PhysicsConstraintComponent)

#define LOCTEXT_NAMESPACE "ConstraintComponent"

DECLARE_LOG_CATEGORY_EXTERN(LogPhysicsConstraints, Log, All);
DEFINE_LOG_CATEGORY(LogPhysicsConstraints);

UPhysicsConstraintComponent::UPhysicsConstraintComponent(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bWantsInitializeComponent = true;
#if WITH_EDITORONLY_DATA
	bVisualizeComponent = true;
#endif
	bIsBrokenLocal = false;
}



UPrimitiveComponent* UPhysicsConstraintComponent::GetComponentInternal(EConstraintFrame::Type Frame) const
{
	UPrimitiveComponent* PrimComp = NULL;

	FName ComponentName = NAME_None;
	AActor* Actor = NULL;

	// Frame 1
	if(Frame == EConstraintFrame::Frame1)
	{
		// Use override component if specified
		if(OverrideComponent1.IsValid())
		{
#if UE_WITH_REMOTE_OBJECT_HANDLE
			if (OverrideComponent1.GetResidence() == EResidence::Remote && !AutoRTFM::IsClosed())
			{
				ensureMsgf(false, TEXT("UPhysicsConstraintComponent::GetComponentInternal: Frame1 Component is remote. Simulation result is not guaranteed."));
				return NULL;
			}
#endif
			return OverrideComponent1.Get();
		}

		ComponentName = ComponentName1.ComponentName;
#if UE_WITH_REMOTE_OBJECT_HANDLE
		if (ConstraintActor1.GetResidence() == EResidence::Remote && !AutoRTFM::IsClosed())
		{
			ensureMsgf(false, TEXT("UPhysicsConstraintComponent::GetComponentInternal: Frame1 Actor is remote. Simulation result is not guaranteed."));
			return NULL;
		}
#endif
		Actor = ConstraintActor1;
	}
	// Frame 2
	else
	{
		// Use override component if specified
		if(OverrideComponent2.IsValid())
		{
#if UE_WITH_REMOTE_OBJECT_HANDLE
			if (OverrideComponent2.GetResidence() == EResidence::Remote && !AutoRTFM::IsClosed())
			{
				ensureMsgf(false, TEXT("UPhysicsConstraintComponent::GetComponentInternal: Frame2 Component is remote. Simulation result is not guaranteed."));
				return NULL;
			}
#endif
			return OverrideComponent2.Get();
		}

		ComponentName = ComponentName2.ComponentName;
#if UE_WITH_REMOTE_OBJECT_HANDLE
		if (ConstraintActor2.GetResidence() == EResidence::Remote && !AutoRTFM::IsClosed())
		{
			ensureMsgf(false, TEXT("UPhysicsConstraintComponent::GetComponentInternal: Frame2 Actor is remote. Simulation result is not guaranteed."));
			return NULL;
		}
#endif
		Actor = ConstraintActor2;
	}

	// If neither actor nor component name specified, joint to 'world'
	if(Actor != NULL || ComponentName != NAME_None)
	{
		// If no actor specified, but component name is - use Owner
		if(Actor == NULL)
		{
			Actor = GetOwner();
		}

		// If we now have an Actor, lets find a component
		if(Actor != NULL)
		{
			// No name specified, use the root component
			if(ComponentName == NAME_None)
			{
				PrimComp = Cast<UPrimitiveComponent>(Actor->GetRootComponent());
			}
			// Name specified, see if we can find that component..
			else
			{
				for(UActorComponent* Comp : Actor->GetComponents())
				{
					if(Comp->GetFName() == ComponentName)
					{
						if (UChildActorComponent* ChildActorComp = Cast<UChildActorComponent>(Comp))
						{
							if (AActor* ChildActor = ChildActorComp->GetChildActor())
							{
								PrimComp = Cast<UPrimitiveComponent>(ChildActor->GetRootComponent());
							}
						}
						else
						{
							PrimComp = Cast<UPrimitiveComponent>(Comp);
						}
						break;
					}
				}
			}
		}	
	}

	return PrimComp;
}

//Helps to find bone index if bone specified, otherwise find bone index of root body
int32 GetBoneIndexHelper(FName InBoneName, const USkeletalMeshComponent& SkelComp, int32* BodyIndex = nullptr)
{
	FName BoneName = InBoneName;
	const UPhysicsAsset* PhysAsset = SkelComp.GetPhysicsAsset();
	
	if(BoneName == NAME_None)
	{
		//Didn't specify bone name so just use root body
		if (PhysAsset)
		{			
			const int32 RootBodyIndex = SkelComp.FindRootBodyIndex();
			if (PhysAsset->SkeletalBodySetups.IsValidIndex(RootBodyIndex))
			{
				BoneName = PhysAsset->SkeletalBodySetups[RootBodyIndex]->BoneName;
			}
		}
	}

	if(BodyIndex)
	{
		*BodyIndex = PhysAsset ? PhysAsset->FindBodyIndex(BoneName) : INDEX_NONE;
	}

	return SkelComp.GetBoneIndex(BoneName);
}

FTransform UPhysicsConstraintComponent::GetBodyTransformInternal(EConstraintFrame::Type Frame, FName InBoneName) const
{
	UPrimitiveComponent* PrimComp = GetComponentInternal(Frame);
	if (!PrimComp)
	{
		return FTransform::Identity;
	}

	//Use GetComponentTransform() by default for all components
	FTransform ResultTM = FTransform::Identity;

	// Skeletal case
	if (const USkeletalMeshComponent* SkelComp = Cast<USkeletalMeshComponent>(PrimComp))
	{
		const int32 BoneIndex = GetBoneIndexHelper(InBoneName, *SkelComp);
		if (BoneIndex != INDEX_NONE)
		{
			ResultTM = SkelComp->GetBoneTransform(BoneIndex);
		}
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
		else
		{
			FMessageLog("PIE").Warning(FText::Format(LOCTEXT("BadBoneNameToConstraint", "Couldn't find bone {0} for ConstraintComponent {1}."),
				FText::FromName(InBoneName), FText::FromString(GetPathNameSafe(this))));
		}
#endif // !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	}
	else if (Chaos::FPhysicsObject* InitialObject = PrimComp->GetPhysicsObjectByName(InBoneName))
	{
		FLockedReadPhysicsObjectExternalInterface Interface = FPhysicsObjectExternalInterface::LockRead({ &InitialObject, 1 });
		if (Chaos::FPhysicsObject* PhysicsObject = Interface->GetRootObject(InitialObject))
		{
			ResultTM = Interface->GetTransform(PhysicsObject);
		}
		else
		{
			ResultTM = Interface->GetTransform(InitialObject);
		}
	}
	else
	{
		ResultTM = PrimComp->GetComponentTransform();
	}

	return ResultTM;
}

FBox UPhysicsConstraintComponent::GetBodyBoxInternal(EConstraintFrame::Type Frame, FName InBoneName) const
{
	FBox ResultBox(ForceInit);

	UPrimitiveComponent* PrimComp  = GetComponentInternal(Frame);

	// Skeletal case
	if(const USkeletalMeshComponent* SkelComp = Cast<USkeletalMeshComponent>(PrimComp))
	{
		if (const UPhysicsAsset* PhysicsAsset = SkelComp->GetPhysicsAsset())
		{
			int32 BodyIndex;
			const int32 BoneIndex = GetBoneIndexHelper(InBoneName, *SkelComp, &BodyIndex);
			if(BoneIndex != INDEX_NONE && BodyIndex != INDEX_NONE)
			{	
				const FTransform BoneTransform = SkelComp->GetBoneTransform(BoneIndex);
				ResultBox = PhysicsAsset->SkeletalBodySetups[BodyIndex]->AggGeom.CalcAABB(BoneTransform);
			}
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
			else
			{
				FMessageLog("PIE").Warning(FText::Format(LOCTEXT("BadBoneNameToConstraint2", "Couldn't find bone {0} for ConstraintComponent {1}."),
					FText::FromName(InBoneName), FText::FromString(GetPathNameSafe(this))));
			}
#endif // !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
		}
	}
	else if(PrimComp != NULL)
	{
		ResultBox = PrimComp->Bounds.GetBox();
	}

	return ResultBox;
}

FTransform UPhysicsConstraintComponent::GetBodyTransform(EConstraintFrame::Type Frame) const
{
	if(Frame == EConstraintFrame::Frame1)
	{
		return GetBodyTransformInternal(Frame, ConstraintInstance.ConstraintBone1);
	}
	else
	{
		return GetBodyTransformInternal(Frame, ConstraintInstance.ConstraintBone2);
	}
}

FBox UPhysicsConstraintComponent::GetBodyBox(EConstraintFrame::Type Frame) const
{
	if(Frame == EConstraintFrame::Frame1)
	{
		return GetBodyBoxInternal(Frame, ConstraintInstance.ConstraintBone1);
	}
	else
	{
		return GetBodyBoxInternal(Frame, ConstraintInstance.ConstraintBone2);
	}
}

FBodyInstance* UPhysicsConstraintComponent::GetBodyInstance(EConstraintFrame::Type Frame) const
{
	FBodyInstance* Instance = NULL;
	UPrimitiveComponent* PrimComp = GetComponentInternal(Frame);
	if(PrimComp != NULL)
	{
		if(Frame == EConstraintFrame::Frame1)
		{
			Instance = PrimComp->GetBodyInstance(ConstraintInstance.ConstraintBone1);
		}
		else
		{
			Instance = PrimComp->GetBodyInstance(ConstraintInstance.ConstraintBone2);
		}
	}
	return Instance;
}

Chaos::FPhysicsObject* UPhysicsConstraintComponent::GetPhysicsObject(EConstraintFrame::Type Frame) const
{
	UPrimitiveComponent* PrimComp = GetComponentInternal(Frame);
	if (!PrimComp)
	{
		return nullptr;
	}

	const FName BoneName = (Frame == EConstraintFrame::Frame1) ? ConstraintInstance.ConstraintBone1 : ConstraintInstance.ConstraintBone2;
	return PrimComp->GetPhysicsObjectByName(BoneName);
}

/** Wrapper that calls our constraint broken delegate */
void UPhysicsConstraintComponent::OnConstraintBrokenWrapper(int32 ConstraintIndex)
{
	OnConstraintBroken.Broadcast(ConstraintIndex);
}

/** Wrapper that calls our plasticity delegate */
void UPhysicsConstraintComponent::OnPlasticDeformationWrapper(int32 ConstraintIndex)
{
	OnPlasticDeformation.Broadcast(ConstraintIndex);
}

void UPhysicsConstraintComponent::InitComponentConstraint()
{
	// First we convert world space position of constraint into local space frames
	UpdateConstraintFrames();

	// Normally, we'd want to init the constraint using the FPhysicsObject that we get from the component.
	// However, to ensure that we can preserve backward compatible behavior, we first check to see if the
	// FBodyInstance exists first. This way the behavior w.r.t. grabbing welded bodies remains the same.
	FBodyInstance* Body1 = GetBodyInstance(EConstraintFrame::Frame1);
	FBodyInstance* Body2 = GetBodyInstance(EConstraintFrame::Frame2);

	Chaos::FPhysicsObject* Object1 = (Body1 && Body1->IsValidBodyInstance()) ? Body1->GetPhysicsActor()->GetPhysicsObject() : GetPhysicsObject(EConstraintFrame::Frame1);
	Chaos::FPhysicsObject* Object2 = (Body2 && Body2->IsValidBodyInstance()) ? Body2->GetPhysicsActor()->GetPhysicsObject() : GetPhysicsObject(EConstraintFrame::Frame2);

	if (Object1 || Object2)
	{
		ConstraintInstance.InitConstraint(Object1, Object2, GetConstraintScale(), this, FOnConstraintBroken::CreateUObject(this, &UPhysicsConstraintComponent::OnConstraintBrokenWrapper));
	}

	bIsBrokenLocal = false;
}

#if UE_WITH_REMOTE_OBJECT_HANDLE

void UPhysicsConstraintComponent::InitComponentConstraintWithoutUpdatingFrames()
{
	// Only reinitialize constraint on migration when the constraint component is not locally broken 
	if (!bIsBrokenLocal)
	{
		FBodyInstance* Body1 = GetBodyInstance(EConstraintFrame::Frame1);
		FBodyInstance* Body2 = GetBodyInstance(EConstraintFrame::Frame2);

		Chaos::FPhysicsObject* Object1 = (Body1 && Body1->IsValidBodyInstance()) ? Body1->GetPhysicsActor()->GetPhysicsObject() : GetPhysicsObject(EConstraintFrame::Frame1);
		Chaos::FPhysicsObject* Object2 = (Body2 && Body2->IsValidBodyInstance()) ? Body2->GetPhysicsActor()->GetPhysicsObject() : GetPhysicsObject(EConstraintFrame::Frame2);

		if (Object1 || Object2)
		{
			ConstraintInstance.InitConstraint(Object1, Object2, GetConstraintScale(), this, FOnConstraintBroken::CreateUObject(this, &UPhysicsConstraintComponent::OnConstraintBrokenWrapper));
		}
	}
}

#endif

void UPhysicsConstraintComponent::TermComponentConstraint()
{
	ConstraintInstance.TermConstraint();
}

void UPhysicsConstraintComponent::OnConstraintBrokenHandler(FConstraintInstance* BrokenConstraint)
{
	OnConstraintBroken.Broadcast(BrokenConstraint->ConstraintIndex);
}

void UPhysicsConstraintComponent::OnPlasticDeformationHandler(FConstraintInstance* Constraint)
{
	OnPlasticDeformation.Broadcast(Constraint->ConstraintIndex);
}


float UPhysicsConstraintComponent::GetConstraintScale() const
{
	return GetComponentScale().GetAbsMin();
}

void UPhysicsConstraintComponent::GetConstrainedComponents(UPrimitiveComponent*& OutComponent1, FName& OutBoneName1, UPrimitiveComponent*& OutComponent2, FName& OutBoneName2)
{
	OutComponent1 = GetComponentInternal(EConstraintFrame::Frame1);
	OutBoneName1 = ConstraintInstance.ConstraintBone1;

	OutComponent2 = GetComponentInternal(EConstraintFrame::Frame2);
	OutBoneName2 = ConstraintInstance.ConstraintBone2;
}

void UPhysicsConstraintComponent::SetConstrainedComponents(UPrimitiveComponent* Component1, FName BoneName1, UPrimitiveComponent* Component2, FName BoneName2)
{
	if(Component1 != NULL)
	{
		this->ComponentName1.ComponentName = Component1->GetFName();
		OverrideComponent1 = Component1;
		ConstraintInstance.ConstraintBone1 = BoneName1;
	}

	if(Component2 != NULL)
	{
		this->ComponentName2.ComponentName = Component2->GetFName();
		OverrideComponent2 = Component2;
		ConstraintInstance.ConstraintBone2 = BoneName2;
	}

	InitComponentConstraint();
}

void UPhysicsConstraintComponent::BreakConstraint()
{
	ConstraintInstance.TermConstraint();
	bIsBrokenLocal = true;
}


void UPhysicsConstraintComponent::InitializeComponent()
{
	Super::InitializeComponent();
	InitComponentConstraint();
}

#if WITH_EDITOR
void UPhysicsConstraintComponent::OnRegister()
{
	Super::OnRegister();

	// This reinitialization is needed in PIE because the editor can re-register a constraint without calling InitializeComponent.
	if(ConstraintInstance.IsTerminated())
	{
		InitComponentConstraint();
	}

	if (SpriteComponent)
	{
		UpdateSpriteTexture();
		SpriteComponent->SpriteInfo.Category = TEXT("Physics");
		SpriteComponent->SpriteInfo.DisplayName = NSLOCTEXT( "SpriteCategory", "Physics", "Physics" );
	}
}
#endif

void UPhysicsConstraintComponent::EndPlay(const EEndPlayReason::Type EndPlayReason)
{
	Super::EndPlay(EndPlayReason);
	TermComponentConstraint();
}

void UPhysicsConstraintComponent::OnUnregister()
{
	Super::OnUnregister();
	TermComponentConstraint();
#if UE_WITH_REMOTE_OBJECT_HANDLE
	CheckAndUnbindPhysicsStateDelegates();
	UnBindPreTickDelegate();
#endif
}

void UPhysicsConstraintComponent::BeginDestroy()
{
	Super::BeginDestroy();

	// Should not be destroying a constraint component with the constraint still created
	ensure(ConstraintInstance.IsTerminated());
}

void UPhysicsConstraintComponent::PostLoad()
{
	Super::PostLoad();

	// Fix old content that used a ConstraintSetup
	if ( GetLinkerUEVersion() < VER_UE4_ALL_PROPS_TO_CONSTRAINTINSTANCE && (ConstraintSetup_DEPRECATED != NULL) )
	{
		// Will have copied from setup into DefaultIntance inside
		ConstraintInstance.CopyConstraintParamsFrom(&ConstraintSetup_DEPRECATED->DefaultInstance);
		ConstraintSetup_DEPRECATED = NULL;
	}

	if (GetLinkerUEVersion() < VER_UE4_SOFT_CONSTRAINTS_USE_MASS)
	{
		//In previous versions the mass was placed into the spring constant. This is correct because you use different springs for different mass - however, this makes tuning hard
		//We now multiply mass into the spring constant. To fix old data we use CalculateMass which is not perfect but close (within 0.1kg)
		//We also use the primitive body instance directly for determining if simulated - this is potentially wrong for fixed bones in skeletal mesh, but it's much more likely right (in skeletal case we don't have access to bodies to check)
		
		UPrimitiveComponent * Primitive1 = GetComponentInternal(EConstraintFrame::Frame1);
		UPrimitiveComponent * Primitive2 = GetComponentInternal(EConstraintFrame::Frame2);
		
		int NumDynamic = 0;
		float TotalMass = 0.f;

		if (Primitive1 && Primitive1->BodyInstance.bSimulatePhysics)
		{
			FName BoneName = ConstraintInstance.ConstraintBone1;
			++NumDynamic;
			TotalMass += Primitive1->CalculateMass(BoneName);
		}

		if (Primitive2 && Primitive2->BodyInstance.bSimulatePhysics)
		{
			FName BoneName = ConstraintInstance.ConstraintBone2;
			++NumDynamic;
			TotalMass += Primitive2->CalculateMass(BoneName);
		}

		if ( (NumDynamic > 0) && (TotalMass > 0) )	//we don't support cases where both constrained bodies are static or NULL, but add this anyway to avoid crash
		{
			float AverageMass = TotalMass / NumDynamic;

#if WITH_EDITORONLY_DATA
			ConstraintInstance.ProfileInstance.LinearLimit.Stiffness /= AverageMass;
			ConstraintInstance.SwingLimitStiffness_DEPRECATED /= AverageMass;
			ConstraintInstance.TwistLimitStiffness_DEPRECATED /= AverageMass;
			ConstraintInstance.LinearLimitDamping_DEPRECATED /= AverageMass;
			ConstraintInstance.SwingLimitDamping_DEPRECATED /= AverageMass;
			ConstraintInstance.TwistLimitDamping_DEPRECATED /= AverageMass;
			
#endif
		}

	}
}

#if WITH_EDITOR
void UPhysicsConstraintComponent::PostEditChangeChainProperty(FPropertyChangedChainEvent& PropertyChangedEvent)
{
	ConstraintInstance.ProfileInstance.SyncChangedConstraintProperties(PropertyChangedEvent);
	Super::PostEditChangeChainProperty(PropertyChangedEvent);
}

void UPhysicsConstraintComponent::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	UpdateConstraintFrames();
	UpdateSpriteTexture();

	Super::PostEditChangeProperty(PropertyChangedEvent);
}

void UPhysicsConstraintComponent::PostEditComponentMove(bool bFinished)
{
	Super::PostEditComponentMove(bFinished);

	// Update frames
	UpdateConstraintFrames();
}

void UPhysicsConstraintComponent::CheckForErrors()
{
	Super::CheckForErrors();

	UPrimitiveComponent* PrimComp1 = GetComponentInternal(EConstraintFrame::Frame1);
	UPrimitiveComponent* PrimComp2 = GetComponentInternal(EConstraintFrame::Frame2);

	// Check we have something to joint
	if( PrimComp1 == NULL && PrimComp2 == NULL )
	{
		FFormatNamedArguments Arguments;
		Arguments.Add(TEXT("OwnerName"), FText::FromString(GetNameSafe(GetOwner())));
		FMessageLog("MapCheck").Warning()
			->AddToken(FUObjectToken::Create(this))
			->AddToken(FTextToken::Create(FText::Format( LOCTEXT("NoComponentsFound","{OwnerName} : No components found to joint."), Arguments ) ));
	}
	// Make sure constraint components are not both static.
	else if ( PrimComp1 != NULL && PrimComp2 != NULL )
	{
		if ( PrimComp1->Mobility != EComponentMobility::Movable && PrimComp2->Mobility != EComponentMobility::Movable )
		{
			FFormatNamedArguments Arguments;
			Arguments.Add(TEXT("OwnerName"), FText::FromString(GetNameSafe(GetOwner())));
			FMessageLog("MapCheck").Warning()
				->AddToken(FUObjectToken::Create(this))
				->AddToken(FTextToken::Create(FText::Format( LOCTEXT("BothComponentsStatic","{OwnerName} : Both components are static."), Arguments ) ));
		}
	}
	else
	{
		// At this point, we know one constraint component is NULL and the other is non-NULL.
		// Check that the non-NULL constraint component is dynamic.
		if ( ( PrimComp1 == NULL && PrimComp2->Mobility != EComponentMobility::Movable ) ||
			( PrimComp2 == NULL && PrimComp1->Mobility != EComponentMobility::Movable ) )
		{
			FFormatNamedArguments Arguments;
			Arguments.Add(TEXT("OwnerName"), FText::FromString(GetNameSafe(GetOwner())));
			FMessageLog("MapCheck").Warning()
				->AddToken(FUObjectToken::Create(this))
				->AddToken(FTextToken::Create(FText::Format( LOCTEXT("SingleStaticComponent","{OwnerName} : Connected to single static component."), Arguments ) ));
		}
	}
}

#endif // WITH_EDITOR

void UPhysicsConstraintComponent::UpdateConstraintFrames()
{
	FTransform A1Transform = GetBodyTransform(EConstraintFrame::Frame1);
	A1Transform.RemoveScaling();

	FTransform A2Transform = GetBodyTransform(EConstraintFrame::Frame2);
	A2Transform.RemoveScaling();

	// World ref frame
	const FVector WPos = GetComponentLocation();
	const FVector WPri = GetComponentTransform().GetUnitAxis( EAxis::X );
	const FVector WOrth = GetComponentTransform().GetUnitAxis( EAxis::Y );

	ConstraintInstance.Pos1 = A1Transform.InverseTransformPosition(WPos);
	ConstraintInstance.PriAxis1 = A1Transform.InverseTransformVectorNoScale(WPri);
	ConstraintInstance.SecAxis1 = A1Transform.InverseTransformVectorNoScale(WOrth);

	const FVector RotatedX = ConstraintInstance.AngularRotationOffset.RotateVector(FVector(1,0,0));
	const FVector RotatedY = ConstraintInstance.AngularRotationOffset.RotateVector(FVector(0,1,0));
	const FVector WPri2 = GetComponentTransform().TransformVectorNoScale(RotatedX);
	const FVector WOrth2 = GetComponentTransform().TransformVectorNoScale(RotatedY);
	
	
	ConstraintInstance.Pos2 = A2Transform.InverseTransformPosition(WPos);
	ConstraintInstance.PriAxis2 = A2Transform.InverseTransformVectorNoScale(WPri2);
	ConstraintInstance.SecAxis2 = A2Transform.InverseTransformVectorNoScale(WOrth2);

	//Constraint instance is given our reference frame scale and uses it to scale position.
	//Note that the scale passed in is also used for limits, so we first undo the position scale so that it's consistent.

	//Note that in the case where there is no body instance, the position is given in world space and there is no scaling.
	const float RefScale = FMath::Max(GetConstraintScale(), 0.01f);
	if(GetPhysicsObject(EConstraintFrame::Frame1))
	{
		ConstraintInstance.Pos1 /= RefScale;
	}

	if (GetPhysicsObject(EConstraintFrame::Frame2))
	{
		ConstraintInstance.Pos2 /= RefScale;
	}
}

void UPhysicsConstraintComponent::SetConstraintReferenceFrame(EConstraintFrame::Type Frame, const FTransform& RefFrame)
{
	ConstraintInstance.SetRefFrame(Frame, RefFrame);
}

void UPhysicsConstraintComponent::SetConstraintReferencePosition(EConstraintFrame::Type Frame, const FVector& RefPosition)
{
	ConstraintInstance.SetRefPosition(Frame, RefPosition);
}

void UPhysicsConstraintComponent::SetConstraintReferenceOrientation(EConstraintFrame::Type Frame, const FVector& PriAxis, const FVector& SecAxis)
{
	ConstraintInstance.SetRefOrientation(Frame, PriAxis, SecAxis);
}

void UPhysicsConstraintComponent::GetConstraintForce(FVector& OutLinearForce, FVector& OutAngularForce)
{
	ConstraintInstance.GetConstraintForce(OutLinearForce, OutAngularForce);
}

bool UPhysicsConstraintComponent::IsBroken()
{
	return bIsBrokenLocal || ConstraintInstance.IsBroken();
}

void UPhysicsConstraintComponent::SetDisableCollision(bool bDisableCollision)
{
	ConstraintInstance.SetDisableCollision(bDisableCollision);
}

bool UPhysicsConstraintComponent::IsProjectionEnabled() const
{
	return ConstraintInstance.IsProjectionEnabled();
}

void UPhysicsConstraintComponent::SetProjectionEnabled(bool bInEnabled)
{
	if (bInEnabled)
	{
		ConstraintInstance.EnableProjection();
	}
	else
	{
		ConstraintInstance.DisableProjection();
	}
}

void UPhysicsConstraintComponent::SetProjectionParams(float ProjectionLinearAlpha, float ProjectionAngularAlpha, float ProjectionLinearTolerance, float ProjectionAngularTolerance)
{
	ConstraintInstance.SetProjectionParams(IsProjectionEnabled(), ProjectionLinearAlpha, ProjectionAngularAlpha, ProjectionLinearTolerance, ProjectionAngularTolerance);
}

FConstraintInstanceAccessor UPhysicsConstraintComponent::GetConstraint()
{
	return FConstraintInstanceAccessor(this);
}

#if WITH_EDITOR
static const TCHAR* GPhysicsConstraintHingeSpriteAssetName = TEXT("/Engine/EditorResources/S_KHinge.S_KHinge");
static const TCHAR* GPhysicsConstraintPrismaticSpriteAssetName = TEXT("/Engine/EditorResources/S_KPrismatic.S_KPrismatic");
static const TCHAR* GPhysicsConstraintJointSpriteAssetName = TEXT("/Engine/EditorResources/S_KBSJoint.S_KBSJoint");
#endif

#if WITH_EDITOR
void UPhysicsConstraintComponent::UpdateSpriteTexture()
{
	if (SpriteComponent)
	{
		FCookLoadScope CookLoadScope(ECookLoadType::EditorOnly);
		if (ConstraintUtils::IsHinge(ConstraintInstance))
		{
			SpriteComponent->SetSprite(LoadObject<UTexture2D>(NULL, GPhysicsConstraintHingeSpriteAssetName));
		}
		else if (ConstraintUtils::IsPrismatic(ConstraintInstance))
		{
			SpriteComponent->SetSprite(LoadObject<UTexture2D>(NULL, GPhysicsConstraintPrismaticSpriteAssetName));
		}
		else
		{
			SpriteComponent->SetSprite(LoadObject<UTexture2D>(NULL, GPhysicsConstraintJointSpriteAssetName));
		}
	}
}
#endif // WITH_EDITOR

#if WITH_EDITORONLY_DATA || UE_WITH_REMOTE_OBJECT_HANDLE

void UPhysicsConstraintComponent::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);
#if WITH_EDITOR
	if (Ar.IsSaving() && Ar.IsObjectReferenceCollector() && !Ar.IsCooking())
	{
		FSoftObjectPathSerializationScope EditorOnlyScope(ESoftObjectPathCollectType::EditorOnlyCollect);
		FSoftObjectPath SpriteAssets[]
		{
			FSoftObjectPath(GPhysicsConstraintHingeSpriteAssetName),
			FSoftObjectPath(GPhysicsConstraintPrismaticSpriteAssetName),
			FSoftObjectPath(GPhysicsConstraintJointSpriteAssetName)
		};
		for (FSoftObjectPath& SpriteAsset : SpriteAssets)
		{
			Ar << SpriteAsset;
		}
	}
#endif

#if UE_WITH_REMOTE_OBJECT_HANDLE
	// For remote objects we need non-UProperties to be transferred
	if (Ar.IsMigratingRemoteObjects())
	{
		Ar << OverrideComponent1;
		Ar << OverrideComponent2;
		Ar << bIsBrokenLocal;
		if (!bIsBrokenLocal)
		{
			MigratePhysicsState(Ar);
		}
	}

	// Clear out all bound delegates if the object is being migrated away. 
	if (Ar.IsSaving())
	{
		CheckAndUnbindPhysicsStateDelegates();
		UnBindPreTickDelegate();
	}
#endif
}

#endif

#if UE_WITH_REMOTE_OBJECT_HANDLE

void UPhysicsConstraintComponent::MigratePhysicsState(FArchive& Ar)
{
	const FUObjectMigrationContext* MigrationContext = Ar.GetMigrationContext();
	check(MigrationContext);

	if (Ar.IsLoading())
	{
		Chaos::FSerializedDataBufferPtr MigratedStateData = MakeUnique<Chaos::FSerializedDataBuffer>();

		Ar << (*MigratedStateData);

		const EObjectMigrationRecvType ObjMigrationRecvType = MigrationContext->GetObjectMigrationRecvType(this);

		if (MigratedStateData->GetDataAsByteArrayRef().IsEmpty())
		{
			UE_LOGF(LogPhysicsConstraints, Verbose, "Migrated an empty physics state (this can be expected) | [%ls] will use its default physics state or the one already had | Migration Context [%ls].", *GetPathNameSafe(this), ToString(ObjMigrationRecvType));
			return;
		}

		if (const bool bFinishedDestroy = HasAnyFlags(EObjectFlags::RF_MirroredGarbage))
		{
			return;
		}

		FPhysScene* PhysicsScene = FPhysicsInterface::GetCurrentSceneForComponent(this);
		if (ensure(PhysicsScene))
		{
			PhysicsScene->PushPendingSerializedStateForObject(this, MoveTemp(MigratedStateData));
		}
	}
	else if (Ar.IsSaving())
	{
		const EObjectMigrationSendType ObjMigrationSendType = MigrationContext->GetObjectMigrationSendType(this);

		FPhysScene* PhysicsScene = FPhysicsInterface::GetCurrentSceneForComponent(this);

		if (CanSendMigratedPhysicsState(MigrationContext, ObjMigrationSendType))
		{
			Chaos::FSerializedMultiPhysicsState MigratedState(ObjMigrationSendType == EObjectMigrationSendType::ReassignOwnership ? Chaos::ESerializedDataContext::Both : Chaos::ESerializedDataContext::External);
			MigratedState.SetNumOfConstraints(1);

			MigratedState.AppendDataFromElement(ConstraintInstance);

			// We need re-serialize the state as a chaos data buffer, so we can use it as is on de-serialization
			Chaos::FSerializedDataBuffer MigratedStateAsBytes;
			FMemoryWriter MigratedStateWriter(MigratedStateAsBytes.GetDataAsByteArrayRef());
			MigratedState.Serialize(MigratedStateWriter);

			Ar << MigratedStateAsBytes;
		}
		else
		{
			// This object might be being migrated before we had a chance to process a previous migration
			Chaos::FSerializedDataBufferPtr RawMigratedData = PhysicsScene ? PhysicsScene->PopPendingSerializedStateForObject(this) : nullptr;

			if (RawMigratedData)
			{
				UE_LOGF(LogPhysicsConstraints, Verbose, "Sending a pending physics state | [%ls]. It is sent away before processed. | Migration Context [%ls].", *GetPathNameSafe(this), ToString(ObjMigrationSendType));
				Ar << *RawMigratedData;
			}
			else
			{
				// We need to serialize an empty buffer as array, so we can handle this case during the load phase post migration
				Chaos::FSerializedDataBuffer EmptyPhysicsState;
				Ar << EmptyPhysicsState;
			}
		}
	}
}

bool UPhysicsConstraintComponent::AreConstrainedComponentsInSameActor() const
{
	if (OverrideComponent1.GetResidence() == EResidence::Remote || OverrideComponent2.GetResidence() == EResidence::Remote)
	{
		return false;
	}

	if (UPrimitiveComponent* Comp1 = OverrideComponent1.Get())
	{
		if (Comp1 == NULL || Comp1->GetOwner() != GetOwner())
		{
			return false;
		}
	}

	if (UPrimitiveComponent* Comp2 = OverrideComponent2.Get())
	{
		if (Comp2 == NULL || Comp2->GetOwner() != GetOwner())
		{
			return false;
		}
	}

	if (ConstraintActor1 != NULL)
	{
		if (ConstraintActor1.GetResidence() == EResidence::Remote)
		{
			return false;
		}
		if (ConstraintActor1.Get() != GetOwner())
		{
			return false;
		}
	}

	if (ConstraintActor2 != NULL)
	{
		if (ConstraintActor2.GetResidence() == EResidence::Remote)
		{
			return false;
		}
		if (ConstraintActor2.Get() != GetOwner())
		{
			return false;
		}
	}

	return true;
}

void UPhysicsConstraintComponent::CheckAndUpdateConstraintPT()
{
	if (HasValidPhysicsState())
	{
		UE_LOGF(LogPhysicsConstraints, Verbose, "UPhysicsConstraintComponent::CheckAndUpdateConstraintPT(): Constraint [%ls] has a valid physics state.", *GetPathNameSafe(this));

		bool bDependentBodiesAreNotMigrated = true;
		bool bIsComp1Migrated = false;
		bool bIsComp2Migrated = false;

		if (UWorld* World = GetWorld())
		{
			if (FPhysScene* PhysScene = World->GetPhysicsScene())
			{
				if (UPrimitiveComponent* Comp1 = GetComponentInternal(EConstraintFrame::Frame1))
				{
					bIsComp1Migrated = PhysScene->IsComponentMigrated(Comp1);
				}
				if (UPrimitiveComponent* Comp2 = GetComponentInternal(EConstraintFrame::Frame2))
				{
					bIsComp2Migrated = PhysScene->IsComponentMigrated(Comp2);
				}
			}
		}

		if (bIsComp1Migrated || bIsComp2Migrated)
		{
			UE_LOGF(LogPhysicsConstraints, Verbose, "UPhysicsConstraintComponent::CheckAndUpdateConstraintPT(): Constraint [%ls] needs to be reinitialized because its PT is no longer valid.", *GetPathNameSafe(this));
			InitComponentConstraintWithoutUpdatingFrames();
		}
		else
		{
			UE_LOGF(LogPhysicsConstraints, Verbose, "UPhysicsConstraintComponent::CheckAndUpdateConstraintPT(): Constraint [%ls] does not need to be reinitialized.", *GetPathNameSafe(this));
		}
	}
}

bool UPhysicsConstraintComponent::CanReInitConstraintAndApplyPhysicsState() const
{
	FBodyInstance* Body1 = GetBodyInstance(EConstraintFrame::Frame1);
	FBodyInstance* Body2 = GetBodyInstance(EConstraintFrame::Frame2);

	Chaos::FPhysicsObject* Object1 = (Body1 && Body1->IsValidBodyInstance()) ? Body1->GetPhysicsActor()->GetPhysicsObject() : GetPhysicsObject(EConstraintFrame::Frame1);
	Chaos::FPhysicsObject* Object2 = (Body2 && Body2->IsValidBodyInstance()) ? Body2->GetPhysicsActor()->GetPhysicsObject() : GetPhysicsObject(EConstraintFrame::Frame2);

	return (((!Body1) == (!Object1)) && ((!Body2) == (!Object2)));
}

void UPhysicsConstraintComponent::PostMigrate(const struct FUObjectMigrationContext& MigrationContext)
{
	Super::PostMigrate(MigrationContext);

	if (MigrationContext.MigrationSide == EObjectMigrationSide::Receive)
	{
		EObjectMigrationRecvType RecvType = MigrationContext.GetObjectMigrationRecvType(this);

		if (!bIsBrokenLocal)
		{
			if (HasValidPhysicsState())
			{
				ApplyMigratedPhysicsState();
			}
			else if (IsValid(this))
			{
				if (AreConstrainedComponentsInSameActor())
				{
					if (CanReInitConstraintAndApplyPhysicsState())
					{
						InitComponentConstraintWithoutUpdatingFrames();
						ApplyMigratedPhysicsState();
					}
					else
					{
						AddReInitDelegatesToComponents();
					}
				}
				else
				{
					if (UWorld* World = GetWorld())
					{
						if (FPhysScene* PhysScene = World->GetPhysicsScene())
						{
							PreTickDelegateHandle = PhysScene->OnPhysScenePreTick.AddUObject(this, &UPhysicsConstraintComponent::ReInitConstraintAndApplyPhysicsState);
						}
					}
				}
			}
		}
	}
}

void UPhysicsConstraintComponent::ReInitConstraintAndApplyPhysicsState(FPhysScene* PhysScene, float DeltaTime)
{
#if UE_WITH_REMOTE_OBJECT_HANDLE
	if (CanReInitConstraintAndApplyPhysicsState())
	{
		UE_LOGF(LogPhysicsConstraints, Verbose, "UPhysicsConstraintComponent::ReInitConstraintAndApplyPhysicsState: Reinitializing constraint and apply migrated physics state ");
		InitComponentConstraintWithoutUpdatingFrames();
		ApplyMigratedPhysicsState();
		UnBindPreTickDelegate();
	}
#endif
}


void UPhysicsConstraintComponent::AddReInitDelegatesToComponents()
{
	if (UPrimitiveComponent* Comp1 = GetComponentInternal(EConstraintFrame::Frame1))
	{
		Comp1->OnComponentPhysicsStateChanged.AddUniqueDynamic(this, &UPhysicsConstraintComponent::ReInitConstraintAndApplyPhysicsState);
	}
	if (UPrimitiveComponent* Comp2 = GetComponentInternal(EConstraintFrame::Frame2))
	{
		Comp2->OnComponentPhysicsStateChanged.AddUniqueDynamic(this, &UPhysicsConstraintComponent::ReInitConstraintAndApplyPhysicsState);
	}
}

void UPhysicsConstraintComponent::ApplyMigratedPhysicsState(Chaos::FSerializedDataBufferPtr MigratedStateData)
{
	Chaos::FSerializedDataBufferPtr RawMigratedData;

	if (MigratedStateData)
	{
		RawMigratedData = MoveTemp(MigratedStateData);
	}
	else if (FPhysScene* PhysicsScene = FPhysicsInterface::GetCurrentSceneForComponent(this))
	{
		RawMigratedData = PhysicsScene->PopPendingSerializedStateForObject(this);
	}

	if (RawMigratedData)
	{
		FMemoryReader MultiBodyStateReader(RawMigratedData->GetDataAsByteArrayRef());

		Chaos::FSerializedMultiPhysicsState MigratedState;
		MigratedState.Serialize(MultiBodyStateReader);

		const Chaos::FSerializedMultiPhysicsState::FHeader& MigratedStateHeader = MigratedState.GetHeader();
		const bool bIsValidMigratedState = MigratedStateHeader.IsValid() && MigratedStateHeader.NumConstraints == 1;
		if (!ensure(bIsValidMigratedState))
		{
			return;
		}

		UE_LOGF(LogPhysicsConstraints, Verbose, "[%s] Applying migrated physics state | Data context [%ls]", __func__, Chaos::LexToString(MigratedState.GetHeader().DataContext));

		if (ensure(ConstraintInstance.IsValidConstraintInstance()))
		{
			MigratedState.ApplyDataToElement(ConstraintInstance);
		}
	}
}

bool UPhysicsConstraintComponent::CanSendMigratedPhysicsState(const FUObjectMigrationContext* MigrationContext, EObjectMigrationSendType ObjMigrationSendType) const
{
	if (!HasValidPhysicsState())
	{
		return false;
	}

	if (ObjMigrationSendType == EObjectMigrationSendType::ReturnBorrowed)
	{
		return MigrationContext->MultiServerCommitRequestId.IsValid();
	}

	return true;
}

bool UPhysicsConstraintComponent::HasValidPhysicsState() const
{
	return ConstraintInstance.IsValidConstraintInstance();
}

void UPhysicsConstraintComponent::CheckAndUnbindPhysicsStateDelegates(bool bCheckIfBound)
{
	TFunction<void(UPrimitiveComponent*, const FString&)> CheckAndUnbind =
		[this, bCheckIfBound](UPrimitiveComponent* Comp, const FString& FrameLabel)
		{
			if (Comp)
			{
				if (Comp->OnComponentPhysicsStateChanged.IsAlreadyBound(this, &UPhysicsConstraintComponent::ReInitConstraintAndApplyPhysicsState))
				{
					if (bCheckIfBound)
					{
						UE_LOGF(LogPhysics, Warning,
							"UPhysicsConstraintComponent::CheckAndUnbindPhysicsStateDelegates: PhysicsConstraintComponent [%ls] still had physics state delegate bound for %ls. Cleaning up.",
							*GetNameSafe(this), *FrameLabel);
					}

					Comp->OnComponentPhysicsStateChanged.RemoveDynamic(this, &UPhysicsConstraintComponent::ReInitConstraintAndApplyPhysicsState);
				}
			}
		};
	// This function is called inside Serialize as well, which is inside the migration code
	// We shouldn't borrow when migrating objects, so the check will skip the unbind if the dependent bodies may need to be borrowed. 
	if (AreConstrainedComponentsInSameActor())
	{
		CheckAndUnbind(GetComponentInternal(EConstraintFrame::Frame1), TEXT("Frame1"));
		CheckAndUnbind(GetComponentInternal(EConstraintFrame::Frame2), TEXT("Frame2"));
	}
}

void UPhysicsConstraintComponent::UnBindPreTickDelegate()
{
	if (PreTickDelegateHandle.IsValid())
	{
		if (const UWorld* World = GetWorld())
		{
			if (FPhysScene* PhysScene = World->GetPhysicsScene())
			{
				PhysScene->OnPhysScenePreTick.Remove(PreTickDelegateHandle);
				PreTickDelegateHandle.Reset();
			}
		}
	}
}

void UPhysicsConstraintComponent::GetConstrainedBodyIds(TPair<FRemoteServerId, FRemoteServerId>& OwnerServerIds, TPair<FRemoteObjectId, FRemoteObjectId>& ObjectIds) const
{
	UPrimitiveComponent* Comp1 = GetComponentInternal(EConstraintFrame::Frame1);
	UPrimitiveComponent* Comp2 = GetComponentInternal(EConstraintFrame::Frame2);
	if (Comp1 != NULL)
	{
		OwnerServerIds.Key = UE::RemoteObject::Handle::GetOwnerServerId(Comp1);
		ObjectIds.Key = TObjectPtr<AActor>(Comp1->GetOwner()).GetRemoteId();
	}

	if (Comp2 != NULL)
	{
		OwnerServerIds.Value = UE::RemoteObject::Handle::GetOwnerServerId(Comp2);
		ObjectIds.Value = TObjectPtr<AActor>(Comp2->GetOwner()).GetRemoteId();
	}
}
#endif

void UPhysicsConstraintComponent::ReInitConstraintAndApplyPhysicsState(UPrimitiveComponent* ChangedComponent, EComponentPhysicsStateChange StateChange)
{
#if UE_WITH_REMOTE_OBJECT_HANDLE
	if (StateChange == EComponentPhysicsStateChange::Created && CanReInitConstraintAndApplyPhysicsState())
	{
		InitComponentConstraintWithoutUpdatingFrames();
		ApplyMigratedPhysicsState();
		CheckAndUnbindPhysicsStateDelegates(false);
	}
#endif
}

void UPhysicsConstraintComponent::SetLinearPositionDrive( bool bEnableDriveX, bool bEnableDriveY, bool bEnableDriveZ )
{
	ConstraintInstance.SetLinearPositionDrive(bEnableDriveX, bEnableDriveY, bEnableDriveZ);
}

void UPhysicsConstraintComponent::SetLinearVelocityDrive( bool bEnableDriveX, bool bEnableDriveY, bool bEnableDriveZ )
{
	ConstraintInstance.SetLinearVelocityDrive(bEnableDriveX, bEnableDriveY, bEnableDriveZ);
}

void UPhysicsConstraintComponent::SetOrientationDriveTwistAndSwing( bool bEnableTwistDrive, bool bEnableSwingDrive)
{
	ConstraintInstance.SetOrientationDriveTwistAndSwing(bEnableTwistDrive, bEnableSwingDrive);
}

void UPhysicsConstraintComponent::SetOrientationDriveSLERP(bool bEnableSLERP)
{
	ConstraintInstance.SetOrientationDriveSLERP(bEnableSLERP);
}


void UPhysicsConstraintComponent::SetAngularDriveMode(EAngularDriveMode::Type DriveMode)
{
	ConstraintInstance.SetAngularDriveMode(DriveMode);
}

void UPhysicsConstraintComponent::SetAngularDriveAccelerationMode(bool bAccelerationMode)
{
	ConstraintInstance.SetAngularDriveAccelerationMode(bAccelerationMode);
}

void UPhysicsConstraintComponent::SetAngularVelocityDriveTwistAndSwing( bool bEnableTwistDrive, bool bEnableSwingDrive)
{
	ConstraintInstance.SetAngularVelocityDriveTwistAndSwing(bEnableTwistDrive, bEnableSwingDrive);
}

void UPhysicsConstraintComponent::SetAngularVelocityDriveSLERP(bool bEnableSLERP)
{
	ConstraintInstance.SetAngularVelocityDriveSLERP(bEnableSLERP);
}


void UPhysicsConstraintComponent::SetLinearPositionTarget( const FVector& InPosTarget )
{
	ConstraintInstance.SetLinearPositionTarget(InPosTarget);
}

void UPhysicsConstraintComponent::SetLinearVelocityTarget( const FVector& InVelTarget )
{
	ConstraintInstance.SetLinearVelocityTarget(InVelTarget);
}

void UPhysicsConstraintComponent::SetLinearDriveParams( float PositionStrength, float VelocityStrength, float InForceLimit )
{
	ConstraintInstance.SetLinearDriveParams(PositionStrength, VelocityStrength, InForceLimit);
}


void UPhysicsConstraintComponent::SetLinearDriveAccelerationMode(bool bAccelerationMode)
{
	ConstraintInstance.SetLinearDriveAccelerationMode(bAccelerationMode);
}

void UPhysicsConstraintComponent::SetAngularOrientationTarget( const FRotator& InPosTarget )
{
	ConstraintInstance.SetAngularOrientationTarget(InPosTarget.Quaternion());
}

void UPhysicsConstraintComponent::SetAngularVelocityTarget( const FVector& InVelTarget )
{
	ConstraintInstance.SetAngularVelocityTarget(InVelTarget);
}

void UPhysicsConstraintComponent::SetAngularDriveParams( float PositionStrength, float VelocityStrength, float InForceLimit )
{
	ConstraintInstance.SetAngularDriveParams(PositionStrength, VelocityStrength, InForceLimit);
}

void UPhysicsConstraintComponent::SetLinearXLimit(ELinearConstraintMotion Motion, float LinearLimit)
{
	ConstraintInstance.SetLinearXLimit(Motion, LinearLimit);
}

void UPhysicsConstraintComponent::SetLinearYLimit(ELinearConstraintMotion Motion, float LinearLimit)
{
	ConstraintInstance.SetLinearYLimit(Motion, LinearLimit);
}

void UPhysicsConstraintComponent::SetLinearZLimit(ELinearConstraintMotion Motion, float LinearLimit)
{
	ConstraintInstance.SetLinearZLimit(Motion, LinearLimit);
}

void UPhysicsConstraintComponent::SetAngularSwing1Limit(EAngularConstraintMotion Motion, float Swing1LimitAngle)
{
	ConstraintInstance.SetAngularSwing1Limit(Motion, Swing1LimitAngle);
}

void UPhysicsConstraintComponent::SetAngularSwing2Limit(EAngularConstraintMotion Motion, float Swing2LimitAngle)
{
	ConstraintInstance.SetAngularSwing2Limit(Motion, Swing2LimitAngle);
}

void UPhysicsConstraintComponent::SetAngularTwistLimit(EAngularConstraintMotion Motion, float TwistLimitAngle)
{
	ConstraintInstance.SetAngularTwistLimit(Motion, TwistLimitAngle);
}

void UPhysicsConstraintComponent::SetLinearBreakable(bool bLinearBreakable, float LinearBreakThreshold)
{
	ConstraintInstance.SetLinearBreakable(bLinearBreakable, LinearBreakThreshold);
}

void UPhysicsConstraintComponent::SetLinearPlasticity(bool bLinearPlasticity, float LinearPlasticityThreshold, EConstraintPlasticityType PlasticityType)
{
	ConstraintInstance.SetLinearPlasticity(bLinearPlasticity, LinearPlasticityThreshold, PlasticityType);
}

void UPhysicsConstraintComponent::SetAngularBreakable(bool bAngularBreakable, float AngularBreakThreshold)
{
	ConstraintInstance.SetAngularBreakable(bAngularBreakable, AngularBreakThreshold);
}

void UPhysicsConstraintComponent::SetAngularPlasticity(bool bAngularPlasticity, float AngularPlasticityThreshold)
{
	ConstraintInstance.SetAngularPlasticity(bAngularPlasticity, AngularPlasticityThreshold);
}

void UPhysicsConstraintComponent::SetContactTransferScale(float ContactTransferScale)
{
	ConstraintInstance.SetContactTransferScale(ContactTransferScale);
}

float UPhysicsConstraintComponent::GetCurrentTwist() const
{
	const float CurrentTwistRads = ConstraintInstance.GetCurrentTwist();
	return FMath::RadiansToDegrees(CurrentTwistRads);
}

float UPhysicsConstraintComponent::GetCurrentSwing1() const
{
	const float CurrentSwing1Rads = ConstraintInstance.GetCurrentSwing1();
	return FMath::RadiansToDegrees(CurrentSwing1Rads);
}

float UPhysicsConstraintComponent::GetCurrentSwing2() const
{
	const float CurrentSwing2Rads = ConstraintInstance.GetCurrentSwing2();
	return FMath::RadiansToDegrees(CurrentSwing2Rads);
}

#undef LOCTEXT_NAMESPACE

