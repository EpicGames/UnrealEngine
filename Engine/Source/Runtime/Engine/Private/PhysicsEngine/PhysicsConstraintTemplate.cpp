// Copyright Epic Games, Inc. All Rights Reserved.

#include "PhysicsEngine/PhysicsConstraintTemplate.h"
#include "PhysicsEngine/PhysicsAsset.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PhysicsConstraintTemplate)

UPhysicsConstraintTemplate::UPhysicsConstraintTemplate(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
#if WITH_EDITORONLY_DATA
	LinearXMotion_DEPRECATED = LCM_Locked;
	LinearYMotion_DEPRECATED = LCM_Locked;
	LinearZMotion_DEPRECATED = LCM_Locked;

	Pos1_DEPRECATED = FVector(0.0f, 0.0f, 0.0f);
	PriAxis1_DEPRECATED = FVector(1.0f, 0.0f, 0.0f);
	SecAxis1_DEPRECATED = FVector(0.0f, 1.0f, 0.0f);

	Pos2_DEPRECATED = FVector(0.0f, 0.0f, 0.0f);
	PriAxis2_DEPRECATED = FVector(1.0f, 0.0f, 0.0f);
	SecAxis2_DEPRECATED = FVector(0.0f, 1.0f, 0.0f);

	LinearBreakThreshold_DEPRECATED = 300.0f;
	AngularBreakThreshold_DEPRECATED = 500.0f;

	ProjectionLinearTolerance_DEPRECATED = 0.5; // Linear projection when error > 5.0 unreal units
	ProjectionAngularTolerance_DEPRECATED = 10.f; // Angular projection when error > 10 degrees
#endif
}

void UPhysicsConstraintTemplate::Serialize(FArchive& Ar)
{
#if WITH_EDITOR
	FConstraintProfileProperties CurrentProfile = DefaultInstance.ProfileInstance;	//Save off current profile in case they save in editor and we don't want to lose their work
	if(Ar.IsSaving() && !Ar.IsTransacting())
	{
		DefaultInstance.ProfileInstance = DefaultProfile;
	}
#endif

	Super::Serialize(Ar);

	// If old content, copy properties out of setup into instance
	if(Ar.UEVer() < VER_UE4_ALL_PROPS_TO_CONSTRAINTINSTANCE)
	{
		CopySetupPropsToInstance(&DefaultInstance);
	}

	if(!Ar.IsTransacting())
	{
		//Make sure to keep default profile and instance in sync
		if (Ar.IsLoading())
		{
			DefaultProfile = DefaultInstance.ProfileInstance;
		}
#if WITH_EDITOR
		else if(Ar.IsSaving())
		{
			DefaultInstance.ProfileInstance = CurrentProfile;	//recover their settings before we saved
		}
#endif
	}
}

#if WITH_EDITOR
void UPhysicsConstraintTemplate::PostEditChangeChainProperty(FPropertyChangedChainEvent& PropertyChangedEvent)
{
	DefaultInstance.ProfileInstance.SyncChangedConstraintProperties(PropertyChangedEvent);
	Super::PostEditChangeChainProperty(PropertyChangedEvent);
}

void UPhysicsConstraintTemplate::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	//If anything changes, update the profile instance
	UpdateProfileInstance();
}

void UPhysicsConstraintTemplate::UpdateProfileInstance()
{
	const FName CurrentProfileName = GetCurrentConstraintProfileName();
	if(CurrentProfileName == NAME_None)
	{
		DefaultProfile = DefaultInstance.ProfileInstance;
	}
	else
	{
		for (FPhysicsConstraintProfileHandle& ProfileHandle : ProfileHandles)
		{
			if (ProfileHandle.ProfileName == CurrentProfileName)
			{
				ProfileHandle.ProfileProperties = DefaultInstance.ProfileInstance;
			}
		}
	}
}

void UPhysicsConstraintTemplate::UpdateConstraintProfiles(const TArray<FName>& Profiles)
{
	for (int32 ProfileIdx = ProfileHandles.Num() - 1; ProfileIdx >= 0; --ProfileIdx)
	{
		if (Profiles.Contains(ProfileHandles[ProfileIdx].ProfileName) == false)
		{
			ProfileHandles.RemoveAtSwap(ProfileIdx);
		}
	}
}

void UPhysicsConstraintTemplate::RenameConstraintProfile(FName CurrentName, FName NewName)
{
	for(FPhysicsConstraintProfileHandle& Handle : ProfileHandles)
	{
		if(Handle.ProfileName == CurrentName)
		{
			Handle.ProfileName = NewName;
			break;
		}
	}
}

void UPhysicsConstraintTemplate::DuplicateConstraintProfile(FName DuplicateFromName, FName DuplicateToName)
{
	for (FPhysicsConstraintProfileHandle& Handle : ProfileHandles)
	{
		if (Handle.ProfileName == DuplicateFromName)
		{
			FPhysicsConstraintProfileHandle* Duplicate = new (ProfileHandles) FPhysicsConstraintProfileHandle(Handle);
			Duplicate->ProfileName = DuplicateToName;
			break;
		}
	}
}

FName UPhysicsConstraintTemplate::GetCurrentConstraintProfileName() const
{
	FName CurrentProfileName;
	if (UPhysicsAsset* OwningPhysAsset = Cast<UPhysicsAsset>(GetOuter()))
	{
		CurrentProfileName = OwningPhysAsset->CurrentConstraintProfileName;
	}

	return CurrentProfileName;
}

#endif

void UPhysicsConstraintTemplate::CopySetupPropsToInstance(FConstraintInstance* Instance)
{
#if WITH_EDITORONLY_DATA
	Instance->JointName			= JointName_DEPRECATED;
	Instance->ConstraintBone1	= ConstraintBone1_DEPRECATED;
	Instance->ConstraintBone2	= ConstraintBone2_DEPRECATED;

	Instance->Pos1				= Pos1_DEPRECATED;
	Instance->PriAxis1			= PriAxis1_DEPRECATED;
	Instance->SecAxis1			= SecAxis1_DEPRECATED;
	Instance->Pos2				= Pos2_DEPRECATED;
	Instance->PriAxis2			= PriAxis2_DEPRECATED;
	Instance->SecAxis2			= SecAxis2_DEPRECATED;

	Instance->ProfileInstance.bEnableProjection				= bEnableProjection_DEPRECATED;
	Instance->ProfileInstance.ProjectionLinearTolerance		= ProjectionLinearTolerance_DEPRECATED;
	Instance->ProfileInstance.ProjectionAngularTolerance		= ProjectionAngularTolerance_DEPRECATED;
	Instance->ProfileInstance.LinearLimit.XMotion				= LinearXMotion_DEPRECATED;
	Instance->ProfileInstance.LinearLimit.YMotion				= LinearYMotion_DEPRECATED;
	Instance->ProfileInstance.LinearLimit.ZMotion				= LinearZMotion_DEPRECATED;
	Instance->ProfileInstance.LinearLimit.Limit				= LinearLimitSize_DEPRECATED;
	Instance->ProfileInstance.LinearLimit.bSoftConstraint		= bLinearLimitSoft_DEPRECATED;
	Instance->ProfileInstance.LinearLimit.Stiffness			= LinearLimitStiffness_DEPRECATED;
	Instance->ProfileInstance.LinearLimit.Damping				= LinearLimitDamping_DEPRECATED;
	Instance->ProfileInstance.bLinearBreakable				= bLinearBreakable_DEPRECATED;
	Instance->ProfileInstance.LinearBreakThreshold			= LinearBreakThreshold_DEPRECATED;
	Instance->ProfileInstance.ConeLimit.Swing1Motion			= AngularSwing1Motion_DEPRECATED;
	Instance->ProfileInstance.ConeLimit.Swing2Motion			= AngularSwing2Motion_DEPRECATED;
	Instance->ProfileInstance.TwistLimit.TwistMotion			= AngularTwistMotion_DEPRECATED;
	Instance->ProfileInstance.ConeLimit.bSoftConstraint		= bSwingLimitSoft_DEPRECATED;
	Instance->ProfileInstance.TwistLimit.bSoftConstraint		= bTwistLimitSoft_DEPRECATED;
	Instance->ProfileInstance.ConeLimit.Swing1LimitDegrees	= Swing1LimitAngle_DEPRECATED;
	Instance->ProfileInstance.ConeLimit.Swing2LimitDegrees	= Swing2LimitAngle_DEPRECATED;
	Instance->ProfileInstance.TwistLimit.TwistLimitDegrees	= TwistLimitAngle_DEPRECATED;
	Instance->ProfileInstance.ConeLimit.Stiffness				= SwingLimitStiffness_DEPRECATED;
	Instance->ProfileInstance.ConeLimit.Damping				= SwingLimitDamping_DEPRECATED;
	Instance->ProfileInstance.TwistLimit.Stiffness			= TwistLimitStiffness_DEPRECATED;
	Instance->ProfileInstance.TwistLimit.Damping				= TwistLimitDamping_DEPRECATED;
	Instance->ProfileInstance.bAngularBreakable				= bAngularBreakable_DEPRECATED;
	Instance->ProfileInstance.AngularBreakThreshold			= AngularBreakThreshold_DEPRECATED;
#endif
}

