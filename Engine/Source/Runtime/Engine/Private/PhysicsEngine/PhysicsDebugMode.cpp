// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_EDITOR

#include "Physics/PhysicsDebugMode.h"

FPhysicsDebugMode::FPhysicsDebugMode() {}
FPhysicsDebugMode::~FPhysicsDebugMode() {}

void FPhysicsDebugMode::Initialize() {}
void FPhysicsDebugMode::Enter() {}
void FPhysicsDebugMode::Exit() {}
void FPhysicsDebugMode::Tick(FEditorViewportClient* ViewportClient, float DeltaTime) {}
bool FPhysicsDebugMode::InputKey(FEditorViewportClient* ViewportClient, FViewport* Viewport, FKey Key, EInputEvent Event) { return false; }
bool FPhysicsDebugMode::InputAxis(FEditorViewportClient* ViewportClient, FViewport* Viewport, int32 ControllerId, FKey Key, float Delta, float DeltaTime) { return false; }
bool FPhysicsDebugMode::InputDelta(FEditorViewportClient* ViewportClient, FViewport* Viewport, FVector& Drag, FRotator& Rotation, FVector& Scale) { return false; }
bool FPhysicsDebugMode::IsCompatibleWith(FEditorModeID OtherModeID) const { return true; }
void FPhysicsDebugMode::AddReferencedObjects(FReferenceCollector& Collector) {}
void FPhysicsDebugMode::Render(const FSceneView* SceneView, FViewport* Viewport, FPrimitiveDrawInterface* PDI) {}
void FPhysicsDebugMode::PostUndo() {}
bool FPhysicsDebugMode::FrustumSelect(const FConvexVolume& InFrustum, FEditorViewportClient* InViewportClient, bool InSelect) { return false; }
bool FPhysicsDebugMode::ShouldDrawWidget() const { return true; }

#endif
