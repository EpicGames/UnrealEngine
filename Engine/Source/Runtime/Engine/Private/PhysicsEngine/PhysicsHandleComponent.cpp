// Copyright Epic Games, Inc. All Rights Reserved.

#include "PhysicsEngine/PhysicsHandleComponent.h"
#include "Components/PrimitiveComponent.h"

#include "Chaos/PBDJointConstraintData.h"
#include "Chaos/Sphere.h"
#include "Physics/Experimental/PhysScene_Chaos.h"
#include "PhysicsProxy/SingleParticlePhysicsProxy.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PhysicsHandleComponent)

UPhysicsHandleComponent::UPhysicsHandleComponent(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, bPendingConstraint(false)
	, PhysicsUserData(&ConstraintInstance)
	, GrabbedHandle(nullptr)
	, KinematicHandle(nullptr)
	, ConstraintLocalPosition(FVector::ZeroVector)
	, ConstraintLocalRotation(FRotator::ZeroRotator)
{
	bAutoActivate = true;
	PrimaryComponentTick.bCanEverTick = true;
	PrimaryComponentTick.TickGroup = TG_PrePhysics;
	LinearDamping = 200.0f;
	LinearStiffness = 750.0f;
	AngularDamping = 500.0f;
	AngularStiffness = 1500.0f;
	InterpolationSpeed = 50.f;
	bSoftAngularConstraint = true;
	bSoftLinearConstraint = true;
	bInterpolateTarget = true;
}

void UPhysicsHandleComponent::OnUnregister()
{
	if(GrabbedComponent)
	{
		ReleaseComponent();
	}

	Super::OnUnregister();
}

void UPhysicsHandleComponent::GrabComponent(class UPrimitiveComponent* InComponent, FName InBoneName, FVector GrabLocation, bool bInConstrainRotation)
{
	//Old behavior was automatically using grabbed body's orientation. This is an edge case that we'd rather not support automatically. We do it here for backwards compat

	if (!InComponent)
	{
		return;
	}

	// Get the PxRigidDynamic that we want to grab.
	FBodyInstance* BodyInstance = InComponent->GetBodyInstance(InBoneName);
	if (!BodyInstance)
	{
		return;
	}

	FRotator GrabbedRotation = FRotator::ZeroRotator;

	if(FPhysicsInterface::IsValid(BodyInstance->GetPhysicsActor()))
	{
		FPhysicsCommand::ExecuteRead(BodyInstance->GetPhysicsActor(), [&](const FPhysicsActorHandle& Actor)
		{
			GrabbedRotation = FPhysicsInterface::GetGlobalPose_AssumesLocked(Actor).Rotator();
		});
	}

	GrabComponentImp(InComponent, InBoneName, GrabLocation, GrabbedRotation, bInConstrainRotation);
}

void UPhysicsHandleComponent::GrabComponentAtLocation(class UPrimitiveComponent* Component, FName InBoneName, FVector GrabLocation)
{
	GrabComponentImp(Component, InBoneName, GrabLocation, FRotator::ZeroRotator, false);
}

void UPhysicsHandleComponent::GrabComponentAtLocationWithRotation(class UPrimitiveComponent* Component, FName InBoneName, FVector GrabLocation, FRotator Rotation)
{
	GrabComponentImp(Component, InBoneName, GrabLocation, Rotation, true);
}

void UPhysicsHandleComponent::GrabComponentImp(UPrimitiveComponent* InComponent, FName InBoneName, const FVector& Location, const FRotator& Rotation, bool bInConstrainRotation)
{
	// If we are already holding something - drop it first.
	if(GrabbedComponent != NULL)
	{
		ReleaseComponent();
	}

	if(!InComponent)
	{
		return;
	}

	// Get the PxRigidDynamic that we want to grab.
	FBodyInstance* BodyInstance = InComponent->GetBodyInstance(InBoneName);
	if (!BodyInstance)
	{
		return;
	}

	// If the grabbed body is welded to another, grab that instead
	if (BodyInstance->WeldParent != nullptr)
	{
		BodyInstance = BodyInstance->WeldParent;
	}

	// simulatable bodies should have handles.
	FPhysicsActorHandle InHandle = BodyInstance->GetPhysicsActor();
	if (!InHandle)
	{
		return;
	}

	// the kinematic rigid body needs to be created before the constraint. 
	if (!KinematicHandle)
	{
		using namespace Chaos;

		FActorCreationParams Params;
		Params.InitialTM = FTransform(Rotation, Location);
		FPhysicsInterface::CreateActor(Params, KinematicHandle);

		KinematicHandle->GetGameThreadAPI().SetGeometry(MakeImplicitObjectPtr<TSphere<FReal, 3>>(TVector<FReal, 3>(0.f), 1000.f));
		KinematicHandle->GetGameThreadAPI().SetObjectState(EObjectStateType::Kinematic);

		if (FPhysScene* Scene = BodyInstance->GetPhysicsScene())
		{
			FPhysicsInterface::AddActorToSolver(KinematicHandle, Scene->GetSolver());
			ConstraintInstance.PhysScene = Scene;
		}
	}

	FTransform KinematicTransform(Rotation, Location);

	// set target and current, so we don't need another "Tick" call to have it right
	TargetTransform = CurrentTransform = KinematicTransform;

	KinematicHandle->GetGameThreadAPI().SetKinematicTarget(KinematicTransform);
	
	FTransform GrabbedTransform(InHandle->GetGameThreadAPI().R(), InHandle->GetGameThreadAPI().X());
	ConstraintLocalPosition = GrabbedTransform.InverseTransformPosition(Location);
	ConstraintLocalRotation = FRotator(GrabbedTransform.InverseTransformRotation(FQuat(Rotation)));

	bRotationConstrained = bInConstrainRotation;
	GrabbedHandle = InHandle; 
	GrabbedComponent = InComponent;
	GrabbedBoneName = InBoneName;
}

void UPhysicsHandleComponent::UpdateDriveSettings()
{
	if (ConstraintHandle.IsValid() && ConstraintHandle.Constraint->IsType(Chaos::EConstraintType::JointConstraintType))
	{
		FPhysicsCommand::ExecuteWrite(ConstraintHandle, [&](const FPhysicsConstraintHandle& InConstraintHandle)
		{
			if (Chaos::FJointConstraint* Constraint = static_cast<Chaos::FJointConstraint*>(ConstraintHandle.Constraint))
			{
				Chaos::EJointMotionType LocationMotionType = bSoftLinearConstraint ? Chaos::EJointMotionType::Free : Chaos::EJointMotionType::Locked;
				Chaos::EJointMotionType RotationMotionType = (bSoftAngularConstraint || !bRotationConstrained) ? Chaos::EJointMotionType::Free : Chaos::EJointMotionType::Locked;

				Constraint->SetCollisionEnabled(false);
				Constraint->SetLinearVelocityDriveEnabled(Chaos::TVec3<bool>(LocationMotionType != Chaos::EJointMotionType::Locked));
				Constraint->SetLinearPositionDriveEnabled(Chaos::TVec3<bool>(LocationMotionType != Chaos::EJointMotionType::Locked));
				Constraint->SetLinearMotionTypesX(LocationMotionType);
				Constraint->SetLinearMotionTypesY(LocationMotionType);
				Constraint->SetLinearMotionTypesZ(LocationMotionType);

				Constraint->SetAngularSLerpPositionDriveEnabled(bRotationConstrained && RotationMotionType != Chaos::EJointMotionType::Locked);
				Constraint->SetAngularSLerpVelocityDriveEnabled(bRotationConstrained && RotationMotionType != Chaos::EJointMotionType::Locked);
				Constraint->SetAngularMotionTypesX(RotationMotionType);
				Constraint->SetAngularMotionTypesY(RotationMotionType);
				Constraint->SetAngularMotionTypesZ(RotationMotionType);
				FTransform GrabConstraintLocalTransform(ConstraintLocalRotation, ConstraintLocalPosition);
				Constraint->SetJointTransforms({ FTransform::Identity , GrabConstraintLocalTransform });

				if (LocationMotionType != Chaos::EJointMotionType::Locked)
				{
					Constraint->SetLinearDriveStiffness(Chaos::FVec3(LinearStiffness));
					Constraint->SetLinearDriveDamping(Chaos::FVec3(LinearDamping));
				}

				if (bRotationConstrained && RotationMotionType != Chaos::EJointMotionType::Locked)
				{
					Constraint->SetAngularDriveStiffness(Chaos::FVec3(AngularStiffness));
					Constraint->SetAngularDriveDamping(Chaos::FVec3(AngularDamping));
				}
			}
		});
	}
}

void UPhysicsHandleComponent::ReleaseComponent()
{
	if (ConstraintHandle.IsValid())
	{
		FPhysicsInterface::ReleaseConstraint(ConstraintHandle);
		bPendingConstraint = false;
	}

	if (GrabbedComponent)
	{
		GrabbedComponent->WakeRigidBody();

		GrabbedComponent = NULL;
		GrabbedBoneName = NAME_None;
		GrabbedHandle = nullptr;
	}

	if (KinematicHandle)
	{
		FChaosEngineInterface::ReleaseActor(KinematicHandle, ConstraintInstance.GetPhysicsScene());
	}

	ConstraintInstance.Reset();
}

UPrimitiveComponent* UPhysicsHandleComponent::GetGrabbedComponent() const
{
	return GrabbedComponent;
}

void UPhysicsHandleComponent::SetTargetLocation(FVector NewLocation)
{
	TargetTransform.SetTranslation(NewLocation);
}

void UPhysicsHandleComponent::SetTargetRotation(FRotator NewRotation)
{
	TargetTransform.SetRotation(NewRotation.Quaternion());
}

void UPhysicsHandleComponent::SetTargetLocationAndRotation(FVector NewLocation, FRotator NewRotation)
{
	TargetTransform = FTransform(NewRotation, NewLocation);
}


void UPhysicsHandleComponent::UpdateHandleTransform(const FTransform& NewTransform)
{
	if (!CurrentTransform.Equals(PreviousTransform))
	{
		FPhysicsCommand::ExecuteWrite(KinematicHandle, [&](const FPhysicsActorHandle& InKinematicHandle)
			{
				KinematicHandle->GetGameThreadAPI().SetKinematicTarget(FTransform(CurrentTransform.GetRotation(), CurrentTransform.GetTranslation()));
			});

		PreviousTransform = CurrentTransform;
	}
}

void UPhysicsHandleComponent::TickComponent(float DeltaTime, enum ELevelTick TickType, FActorComponentTickFunction *ThisTickFunction)
{
	Super::TickComponent(DeltaTime, TickType, ThisTickFunction);

	if (bPendingConstraint)
	{
		if (!ConstraintHandle.IsValid())
			return;
		bPendingConstraint = false;
	}

	if (ConstraintHandle.IsValid())
	{
		if (bInterpolateTarget)
		{
			const float Alpha = FMath::Clamp(DeltaTime * InterpolationSpeed, 0.f, 1.f);
			FTransform C = CurrentTransform;
			FTransform T = TargetTransform;
			C.NormalizeRotation();
			T.NormalizeRotation();
			CurrentTransform.Blend(C, T, Alpha);
		}
		else
		{
			CurrentTransform = TargetTransform;
		}

		UpdateHandleTransform(CurrentTransform);
	}
	else if (KinematicHandle && GrabbedHandle)
	{
		using namespace Chaos;

		ConstraintHandle = FChaosEngineInterface::CreateConstraint(KinematicHandle, GrabbedHandle, FTransform::Identity, FTransform::Identity); // Correct transforms will be set in the update
		if (ConstraintHandle.IsValid() && ConstraintHandle.Constraint->IsType(Chaos::EConstraintType::JointConstraintType))
		{
			if (Chaos::FJointConstraint* Constraint = static_cast<Chaos::FJointConstraint*>(ConstraintHandle.Constraint))
			{
				// need to tie together the instance and the handle for scene read/write locks
				Constraint->SetUserData(&PhysicsUserData/*has a (void*)FConstraintInstanceBase*/);
				ConstraintInstance.ConstraintHandle = ConstraintHandle;

				UpdateDriveSettings();
			}
		}
		bPendingConstraint = true;
	}
}

void UPhysicsHandleComponent::GetTargetLocationAndRotation(FVector& OutLocation, FRotator& OutRotation) const 
{
	OutRotation = TargetTransform.Rotator();
	OutLocation = TargetTransform.GetTranslation();
}

void UPhysicsHandleComponent::SetLinearDamping(float NewLinearDamping)
{
	LinearDamping = NewLinearDamping;
	UpdateDriveSettings();
}

void UPhysicsHandleComponent::SetLinearStiffness(float NewLinearStiffness)
{
	LinearStiffness = NewLinearStiffness;
	UpdateDriveSettings();
}

void UPhysicsHandleComponent::SetAngularDamping(float NewAngularDamping)
{
	AngularDamping = NewAngularDamping;
	UpdateDriveSettings();
}

void UPhysicsHandleComponent::SetAngularStiffness(float NewAngularStiffness)
{
	AngularStiffness = NewAngularStiffness;
	UpdateDriveSettings();
}

void UPhysicsHandleComponent::SetInterpolationSpeed(float NewInterpolationSpeed)
{
	InterpolationSpeed = NewInterpolationSpeed;
}

