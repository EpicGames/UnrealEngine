// Copyright Epic Games, Inc. All Rights Reserved.

#include "Physics/PhysicsInterfaceUtils.h"
#include "CollisionQueryParams.h"
#include "Physics/PhysicsFiltering.h"
#include "PhysicsInterfaceTypesCore.h"

FCollisionFilterData CreateQueryFilterData(const uint8 MyChannel, const bool bTraceComplex, const FCollisionResponseContainer& InCollisionResponseContainer, const struct FCollisionQueryParams& QueryParam, const struct FCollisionObjectQueryParams& ObjectParam, const bool bMultitrace)
{
	Chaos::Filter::FQueryFilterData Result = CreateChaosQueryFilterData(MyChannel, bTraceComplex, InCollisionResponseContainer, QueryParam, ObjectParam, bMultitrace);
	return Chaos::Filter::FQueryFilterBuilder::GetLegacyQueryFilter(Result);
}

Chaos::Filter::FQueryFilterData CreateChaosQueryFilterData(const uint8 MyChannel, const bool bTraceComplex, const FCollisionResponseContainer& InCollisionResponseContainer, const struct FCollisionQueryParams& QueryParam, const struct FCollisionObjectQueryParams& ObjectParam, const bool bMultitrace)
{
	Chaos::EFilterFlags FlagsToSet = bTraceComplex ? Chaos::EFilterFlags::ComplexCollision : Chaos::EFilterFlags::SimpleCollision;
	if (ObjectParam.IsValid())
	{
		FPhysicsObjectQueryFilterBuilder Builder;
		Builder.SetObjectTypes(ObjectParam.GetQueryBitfield64());
		Builder.SetMultiQuery(bMultitrace);
		Builder.SetMaskFilter(ObjectParam.IgnoreMask);
		Builder.SetFlags(FlagsToSet, true);
		return Builder.Build();
	}
	else
	{
		FPhysicsTraceQueryFilterBuilder Builder;
		Builder.SetCollisionChannelIndex((ECollisionChannel)MyChannel);
		Builder.SetResponses(InCollisionResponseContainer);
		Builder.SetMaskFilter(QueryParam.IgnoreMask);
		Builder.SetFlags(FlagsToSet, true);
		return Builder.Build();
	}
}
