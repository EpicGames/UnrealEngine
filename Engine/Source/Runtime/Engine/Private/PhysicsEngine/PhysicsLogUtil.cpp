// Copyright Epic Games, Inc. All Rights Reserved.
#include "PhysicsEngine/PhysicsLogUtil.h"
#include "Components/ActorComponent.h"
#include "Containers/UnrealString.h"
#include "Engine/NetDriver.h"
#include "Engine/PackageMapClient.h"
#include "Engine/World.h"
#include "GameFramework/Actor.h"
#include "Misc/NetworkGuid.h"

namespace PhysicsLogUtil
{
	// A simplified net mode string to make visually paring logs easier
	FString MakeCompactNetModeString(ENetMode NetMode)
	{
		switch (NetMode)
		{
			case NM_DedicatedServer:
			case NM_ListenServer:
				return TEXT("Server");
			case NM_Client:
				return TEXT("Client");
		}
		return ToString(NetMode);
	}

	FString MakeComponentNetIDString(const UActorComponent* Component)
	{
		if (Component != nullptr)
		{
			if (const AActor* Actor = Component->GetOwner())
			{
				if (const UNetDriver* NetDriver = Actor->GetNetDriver())
				{
					if (const TSharedPtr<FNetGUIDCache>& NetGUIDCache = NetDriver->GetNetGuidCache())
					{
						// Network data available - return net mode, net ID, actor and component name
						const ENetMode NetMode = Actor->GetNetMode();
						const FNetworkGUID NetGUID = NetGUIDCache->GetNetGUID(Component);
						return FString::Printf(TEXT("%s %-6s"), *MakeCompactNetModeString(NetMode), *NetGUID.ToString());
					}
				}
			}
		}

		return FString("NULL");
	}

	FString MakeActorNameString(const AActor* Actor)
	{
		return AActor::GetDebugName(Actor);
	}

	FString MakeActorNameString(const UActorComponent* ActorComponent)
	{
		if (ActorComponent != nullptr)
		{
			return AActor::GetDebugName(ActorComponent->GetOwner());
		}

		return FString("NULL");
	}

	FString MakeComponentNameString(const UActorComponent* Component)
	{
		if (Component != nullptr)
		{
			if (const AActor* Actor = Component->GetOwner())
			{
				return FString::Printf(TEXT("%s %s"), *AActor::GetDebugName(Actor), *Component->GetName());
			}

			// No actor - return component name
			return Component->GetName();
		}

		return FString("NULL");
	}
}
