// Copyright Epic Games, Inc. All Rights Reserved.
#include "PhysicsEngine/PhysicsObjectBlueprintLibrary.h"
#include "PhysicsEngine/PhysicsObjectExternalInterface.h"
#include "Chaos/PhysicsObjectCollisionInterface.h"
#include "Chaos/PhysicsObjectInterface.h"
#include "Chaos/PhysicsObjectInternalInterface.h"

#include "PhysicsProxy/ClusterUnionPhysicsProxy.h"
#include "PhysicsProxy/SingleParticlePhysicsProxy.h"

#include "Components/PrimitiveComponent.h"
#include "PhysicsEngine/ClusterUnionComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PhysicsObjectBlueprintLibrary)

FClosestPhysicsObjectResult UPhysicsObjectBlueprintLibrary::GetClosestPhysicsObjectFromWorldLocation(UPrimitiveComponent* Component, const FVector& WorldLocation)
{
	if (!Component)
	{
		return {};
	}

	TArray<Chaos::FPhysicsObject*> PhysicsObjects = Component->GetAllPhysicsObjects();
	FLockedReadPhysicsObjectExternalInterface Interface = FPhysicsObjectExternalInterface::LockRead(PhysicsObjects);
	PhysicsObjects = PhysicsObjects.FilterByPredicate(
		// Maybe allow users to customize the predicate?
		[&Interface](Chaos::FPhysicsObject* Object) {
			// Filter for enabled particles. Other systems should be responsible for attaching the wheel to child particles if the geometry collection breaks (for example).
			return !Interface->AreAllDisabled({ &Object, 1 });
		}
	);
	return Interface->GetClosestPhysicsBodyFromLocation(PhysicsObjects, WorldLocation);
}
bool UPhysicsObjectBlueprintLibrary::ExtractClosestPhysicsObjectResults(const FClosestPhysicsObjectResult& Result, FName& OutName)
{
	if (!Result)
	{
		return false;
	}
	OutName = Result.HitName();
	return true;
}
FTransform UPhysicsObjectBlueprintLibrary::GetPhysicsObjectWorldTransform(UPrimitiveComponent* Component, FName BoneName)
{
	if (!Component)
	{
		return FTransform::Identity;
	}

	if (Chaos::FPhysicsObject* PhysicsObject = Component->GetPhysicsObjectByName(BoneName))
	{
		FLockedReadPhysicsObjectExternalInterface Interface = FPhysicsObjectExternalInterface::LockRead({&PhysicsObject, 1});
		return Interface->GetTransform(PhysicsObject);
	}
	
	return FTransform::Identity;
}

void UPhysicsObjectBlueprintLibrary::ApplyRadialImpulse(UPrimitiveComponent* Component, FVector Origin, float Radius, float Strength, enum ERadialImpulseFalloff Falloff, bool bApplyStrain, float Strain, bool bVelChange, float MinValue, float MaxValue)
{
	if (Component == nullptr || Component->bIgnoreRadialImpulse)
	{
		return;
	}

	if (Chaos::FPhysicsObject* PhysicsObject = Component->GetPhysicsObjectByName(NAME_None))
	{
		TArray<Chaos::FPhysicsObjectHandle>  PhysicsObjects;
		PhysicsObjects.Add(PhysicsObject);
		FLockedWritePhysicsObjectExternalInterface Interface = FPhysicsObjectExternalInterface::LockWrite(PhysicsObjects);
		return Interface->AddRadialImpulse(PhysicsObjects, Origin, Radius, Strength, Falloff, bApplyStrain, Strain, /* bInvalidate */ true, bVelChange, MinValue, MaxValue);
	}
}
