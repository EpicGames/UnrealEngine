// Copyright Epic Games, Inc. All Rights Reserved.
#include "PhysicsEngine/PhysicsObjectExternalInterface.h"

#include "Engine/World.h"
#include "Physics/Experimental/PhysScene_Chaos.h"

FLockedReadPhysicsObjectExternalInterface FPhysicsObjectExternalInterface::LockRead(FChaosScene* Scene)
{
	return FLockedReadPhysicsObjectExternalInterface{ Scene, CreateReadInterface<Chaos::EThreadContext::External>() };
}

FLockedReadPhysicsObjectExternalInterface FPhysicsObjectExternalInterface::LockRead(Chaos::FConstPhysicsObjectHandle InObject)
{
	return LockRead(GetScene(InObject));
}

FLockedReadPhysicsObjectExternalInterface FPhysicsObjectExternalInterface::LockRead(TArrayView<const Chaos::FConstPhysicsObjectHandle> InObjects)
{
	return LockRead(GetScene(InObjects));
}

FLockedWritePhysicsObjectExternalInterface FPhysicsObjectExternalInterface::LockWrite(FChaosScene* Scene)
{
	return FLockedWritePhysicsObjectExternalInterface{ Scene, CreateWriteInterface<Chaos::EThreadContext::External>() };
}

FLockedWritePhysicsObjectExternalInterface FPhysicsObjectExternalInterface::LockWrite(TArrayView<const Chaos::FPhysicsObjectHandle> InObjects)
{
	return LockWrite(GetScene(InObjects));
}

Chaos::FReadPhysicsObjectInterface_External FPhysicsObjectExternalInterface::GetRead_AssumesLocked()
{
	return CreateReadInterface<Chaos::EThreadContext::External>();
}

Chaos::FWritePhysicsObjectInterface_External FPhysicsObjectExternalInterface::GetWrite_AssumesLocked()
{
	return CreateWriteInterface<Chaos::EThreadContext::External>();
}

UPrimitiveComponent* FPhysicsObjectExternalInterface::GetComponentFromPhysicsObject(UWorld* World, Chaos::FPhysicsObjectHandle PhysicsObject)
{
	if (!PhysicsObject || !World)
	{
		return nullptr;
	}

	IPhysicsProxyBase* Proxy = Chaos::FPhysicsObjectInterface::GetProxy({ &PhysicsObject, 1 });
	FPhysScene* Scene = World->GetPhysicsScene();
	if (!Scene || !Proxy)
	{
		return nullptr;
	}

	return Scene->GetOwningComponent<UPrimitiveComponent>(Proxy);
}