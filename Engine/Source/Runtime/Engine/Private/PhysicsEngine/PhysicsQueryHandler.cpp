// Copyright Epic Games, Inc. All Rights Reserved.

#include "Physics/PhysicsQueryHandler.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PhysicsQueryHandler)

bool UPhysicsQueryHandler::Overlap(const EQueryInfo InfoType, const EThreadQueryContext ThreadContext, const UWorld* World, const FOverlapQueryData& OverlapData, const FCommonQueryData& CommonData, TArray<FOverlapResult>& OutOverlaps)
{
	return false;
}

bool UPhysicsQueryHandler::RaycastTest(const EThreadQueryContext ThreadContext, const UWorld* World, const FRayQueryData& RayData, const FCommonQueryData& CommonData, FHitResult& OutHits)
{
	return false;
}

bool UPhysicsQueryHandler::RaycastSingle(const EThreadQueryContext ThreadContext, const UWorld* World, const FRayQueryData& RayData, const FCommonQueryData& CommonData, FHitResult& OutHits)
{
	return false;
}

bool UPhysicsQueryHandler::RaycastMulti(const EThreadQueryContext ThreadContext, const UWorld* World, const FRayQueryData& RayData, const FCommonQueryData& CommonData, TArray<FHitResult>& OutHits)
{
	return false;
}

bool UPhysicsQueryHandler::SweepTest(const EThreadQueryContext ThreadContext, const UWorld* World, const FSweepQueryData& SweepData, const FCommonQueryData& CommonData, FHitResult& OutHits)
{
	return false;
}

bool UPhysicsQueryHandler::SweepSingle(const EThreadQueryContext ThreadContext, const UWorld* World, const FSweepQueryData& SweepData, const FCommonQueryData& CommonData, FHitResult& OutHits)
{
	return false;
}

bool UPhysicsQueryHandler::SweepMulti(const EThreadQueryContext ThreadContext, const UWorld* World, const FSweepQueryData& SweepData, const FCommonQueryData& CommonData, TArray<FHitResult>& OutHits)
{
	return false;
}

void UPhysicsQueryHandler::QueueAsyncOverlap(const FTraceHandle& TraceHandle, const EQueryInfo InfoType, const EThreadQueryContext ThreadContext, const UWorld* World, const FOverlapQueryData& OverlapData, const FCommonQueryData& CommonData)
{
}

void UPhysicsQueryHandler::QueueAsyncRaycast(const FTraceHandle& TraceHandle, const EPhysicsQueryKind QueryKind, const EThreadQueryContext ThreadContext, const UWorld* World, const FRayQueryData& RayData, const FCommonQueryData& CommonData)
{
}

void UPhysicsQueryHandler::QueueAsyncSweep(const FTraceHandle& TraceHandle, const EPhysicsQueryKind QueryKind, const EThreadQueryContext ThreadContext, const UWorld* World, const FSweepQueryData& SweepData, const FCommonQueryData& CommonData)
{
}

bool UPhysicsQueryHandler::AreAsyncRequestsAdded() 
{
	return false;
}

void UPhysicsQueryHandler::VerifyAsyncRequestsAreCompletedOrAbort()
{
}
