// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	PhysicsReplication.cpp: Code for keeping replicated physics objects in sync with the server based on replicated server state data.
=============================================================================*/

#include "PhysicsReplication.h"
#include "PhysicsReplicationLOD.h"
#include "Physics/PhysicsReplicationQuantization.h"
#include "Engine/World.h"
#include "Components/SkeletalMeshComponent.h"
#include "DrawDebugHelpers.h"
#include "Physics/Experimental/PhysScene_Chaos.h"
#include "PhysicsEngine/PhysicsSettings.h"
#include "Chaos/PhysicsObjectInternalInterface.h"
#include "GameFramework/PlayerController.h"
#include "GameFramework/PlayerState.h"
#include "PhysicsProxy/SingleParticlePhysicsProxy.h"
#include "PBDRigidsSolver.h"
#include "Chaos/DebugDrawQueue.h"
#include "Chaos/Particles.h"
#include "Chaos/Island/IslandManager.h"
#include "RewindData.h"


namespace CharacterMovementCVars
{
	extern int32 NetShowCorrections;
	extern float NetCorrectionLifetime;

	int32 SkipPhysicsReplication = 0;
	static FAutoConsoleVariableRef CVarSkipPhysicsReplication(TEXT("p.SkipPhysicsReplication"), SkipPhysicsReplication, TEXT(""));

	float NetPingExtrapolation = -1.0f;
	static FAutoConsoleVariableRef CVarNetPingExtrapolation(TEXT("p.NetPingExtrapolation"), NetPingExtrapolation, TEXT(""));

	float NetPingLimit = -1.f;
	static FAutoConsoleVariableRef CVarNetPingLimit(TEXT("p.NetPingLimit"), NetPingLimit, TEXT(""));

	float ErrorPerLinearDifference = -1.0f;
	static FAutoConsoleVariableRef CVarErrorPerLinearDifference(TEXT("p.ErrorPerLinearDifference"), ErrorPerLinearDifference, TEXT(""));

	float ErrorPerAngularDifference = -1.0f;
	static FAutoConsoleVariableRef CVarErrorPerAngularDifference(TEXT("p.ErrorPerAngularDifference"), ErrorPerAngularDifference, TEXT(""));

	float ErrorAccumulationSeconds = -1.0f;
	static FAutoConsoleVariableRef CVarErrorAccumulation(TEXT("p.ErrorAccumulationSeconds"), ErrorAccumulationSeconds, TEXT(""));

	float ErrorAccumulationDistanceSq = -1.0f;
	static FAutoConsoleVariableRef CVarErrorAccumulationDistanceSq(TEXT("p.ErrorAccumulationDistanceSq"), ErrorAccumulationDistanceSq, TEXT(""));

	float ErrorAccumulationSimilarity = -1.f;
	static FAutoConsoleVariableRef CVarErrorAccumulationSimilarity(TEXT("p.ErrorAccumulationSimilarity"), ErrorAccumulationSimilarity, TEXT(""));

	float MaxLinearHardSnapDistance = -1.f;
	static FAutoConsoleVariableRef CVarMaxLinearHardSnapDistance(TEXT("p.MaxLinearHardSnapDistance"), MaxLinearHardSnapDistance, TEXT(""));

	float MaxRestoredStateError = -1.0f;
	static FAutoConsoleVariableRef CVarMaxRestoredStateError(TEXT("p.MaxRestoredStateError"), MaxRestoredStateError, TEXT(""));

	float PositionLerp = -1.0f;
	static FAutoConsoleVariableRef CVarLinSet(TEXT("p.PositionLerp"), PositionLerp, TEXT(""));

	float LinearVelocityCoefficient = -1.0f;
	static FAutoConsoleVariableRef CVarLinLerp(TEXT("p.LinearVelocityCoefficient"), LinearVelocityCoefficient, TEXT(""));

	float AngleLerp = -1.0f;
	static FAutoConsoleVariableRef CVarAngSet(TEXT("p.AngleLerp"), AngleLerp, TEXT(""));

	float AngularVelocityCoefficient = -1.0f;
	static FAutoConsoleVariableRef CVarAngLerp(TEXT("p.AngularVelocityCoefficient"), AngularVelocityCoefficient, TEXT(""));

	int32 AlwaysHardSnap = 0;
	static FAutoConsoleVariableRef CVarAlwaysHardSnap(TEXT("p.AlwaysHardSnap"), AlwaysHardSnap, TEXT(""));

	int32 AlwaysResetPhysics = 0;
	static FAutoConsoleVariableRef CVarAlwaysResetPhysics(TEXT("p.AlwaysResetPhysics"), AlwaysResetPhysics, TEXT(""));

	int32 ApplyAsyncSleepState = 1;
	static FAutoConsoleVariableRef CVarApplyAsyncSleepState(TEXT("p.ApplyAsyncSleepState"), ApplyAsyncSleepState, TEXT(""));
}

namespace RenderInterpolationCVars
{
	bool bRenderInterpDebugDrawResimTrigger = false;
	static FAutoConsoleVariableRef CVarRenderInterpDebugDrawResimTrigger(TEXT("p.RenderInterp.DebugDraw.ResimTrigger"), bRenderInterpDebugDrawResimTrigger, TEXT("Draw debug lines for physics render interpolation, also needs p.Chaos.DebugDraw.Enabled set"));
	float RenderInterpDebugDrawResimBoxScale= 1.0f;
	static FAutoConsoleVariableRef CVarRenderInterpDebugDrawResimBoxScale(TEXT("p.RenderInterp.DebugDraw.ResimBoxScale"), RenderInterpDebugDrawResimBoxScale, TEXT("Draw debug lines for physics render interpolation, also needs p.Chaos.DebugDraw.Enabled set"));
}

namespace PhysicsReplicationLODCVars
{
	// Defined in PhysicsReplicationLOD.cpp.

	extern int32 TransitionExtrapFrameMin;
	extern float TransitionExtrapFraction;
	extern bool bTransitionModeDebugLog;
}

namespace PhysicsReplicationCVars
{
	int32 SkipSkeletalRepOptimization = 1;
	static FAutoConsoleVariableRef CVarSkipSkeletalRepOptimization(TEXT("p.SkipSkeletalRepOptimization"), SkipSkeletalRepOptimization, TEXT("If true, we don't move the skeletal mesh component during replication. This is ok because the skeletal mesh already polls physx after its results"));
#if !UE_BUILD_SHIPPING
	int32 LogPhysicsReplicationHardSnaps = 0;
	static FAutoConsoleVariableRef CVarLogPhysicsReplicationHardSnaps(TEXT("p.LogPhysicsReplicationHardSnaps"), LogPhysicsReplicationHardSnaps, TEXT(""));
#endif

	int32 EnableDefaultReplication = 0;
	static FAutoConsoleVariableRef CVarEnableDefaultReplication(TEXT("np2.EnableDefaultReplication"), EnableDefaultReplication, TEXT("Enable default replication in the networked physics prediction flow."));

	int32 DebugDrawShowRepMode = 0;
	static FAutoConsoleVariableRef CVarPhysicsNetDebugDrawShowRepMode(TEXT("p.Net.DebugDraw.ShowRepMode"), DebugDrawShowRepMode, TEXT("Debug draw to show which physics replication mode is being used and where target states are being received. Green = Legacy Default, Light Blue = New Default, Yellow = Predictive Interpolation, Red = Resimulation, White = Something's Wrong. NOTE, Requires CVar p.Chaos.DebugDraw.Enabled 1"));

	float DebugDrawLifeTime = 3.0f;
	static FAutoConsoleVariableRef CVarPhysicsNetDebugDrawLifeTime(TEXT("p.Net.DebugDraw.LifeTime"), DebugDrawLifeTime, TEXT("Networked physics debug draw lifetime"));

	namespace DefaultReplicationCVars
	{
		bool bHardsnapLegacyInPT = false;
		static FAutoConsoleVariableRef CVarDefaultLegacyHardsnapInPT(TEXT("p.DefaultReplication.Legacy.HardsnapInPT"), bHardsnapLegacyInPT, TEXT("If default replication is used and it's running the legacy flow through Game Thread, allow hardsnapping to be performed on Physics Thread if async physics is enabled.."));

		bool bCorrectConnectedBodies = false;
		static FAutoConsoleVariableRef CVarDefaultCorrectConnectedBodies(TEXT("p.DefaultReplication.CorrectConnectedBodies"), bCorrectConnectedBodies, TEXT("When true, transform corrections will also apply to any connected physics object."));

		bool bCorrectConnectedBodiesFriction = true;
		static FAutoConsoleVariableRef CVarDefaultCorrectConnectedBodiesFriction(TEXT("p.DefaultReplication.CorrectConnectedBodiesFriction"), bCorrectConnectedBodiesFriction, TEXT("When true, transform correction on any connected physics object will also recalculate their friction."));
	}

	namespace ResimulationCVars
	{
		extern bool bApplyPredictiveInterpolationWhenBehindServer;

		bool bResimulateSleepDesync = true;
		static FAutoConsoleVariableRef CVarResimulateSleepDesync(TEXT("np2.Resim.ResimulateSleepDesync"), bResimulateSleepDesync, TEXT("Set if sleeping discrepancy between client and server should trigger a resimulation. Not enabling this can lead to client desyncing continuously if the server is sleeping and not sending data to client while the client object is still dynamic."));

		bool bRuntimeCorrectionEnabled = false;
		static FAutoConsoleVariableRef CVarResimRuntimeCorrectionEnabled(TEXT("np2.Resim.RuntimeCorrectionEnabled"), bRuntimeCorrectionEnabled, TEXT("Apply positional and rotational runtime corrections while within resim trigger distance."));

		bool bRuntimeVelocityCorrection = false;
		static FAutoConsoleVariableRef CVarResimRuntimeVelocityCorrection(TEXT("np2.Resim.RuntimeVelocityCorrection"), bRuntimeVelocityCorrection, TEXT("Apply linear and angular velocity corrections in runtime while within resim trigger distance. Used if RuntimeCorrectionEnabled is true."));

		bool bRuntimeCorrectConnectedBodies = true;
		static FAutoConsoleVariableRef CVarResimRuntimeCorrectConnectedBodies(TEXT("np2.Resim.RuntimeCorrectConnectedBodies"), bRuntimeCorrectConnectedBodies, TEXT("If true runtime position and rotation correction will also shift transform of any connected physics objects. Used if RuntimeCorrectionEnabled is true."));

		bool bDisableReplicationOnInteraction = false;
		static FAutoConsoleVariableRef CVarResimDisableReplicationOnInteraction(TEXT("np2.Resim.DisableReplicationOnInteraction"), bDisableReplicationOnInteraction, TEXT("If a resim object interacts with another object not running resimulation, deactivate that objects replication until interaction stops."));

		bool bKeepResimStateForNonResimReplicatedObjects = false;
		static FAutoConsoleVariableRef CVarResimKeepResimStateForNonResimReplicatedObjects(TEXT("np2.Resim.KeepResimStateForNonResimReplicatedObjects"), bKeepResimStateForNonResimReplicatedObjects, TEXT("When false, move objects back to their pre-resim state at the end of a resimulation if they are not set to use resimulation as replication mode."));

		float PosStabilityMultiplier = 0.5f;
		static FAutoConsoleVariableRef CVarResimPosStabilityMultiplier(TEXT("np2.Resim.PosStabilityMultiplier"), PosStabilityMultiplier, TEXT("Recommended range between 0.0-1.0. Lower value means more stable positional corrections."));

		float RotStabilityMultiplier = 1.0f;
		static FAutoConsoleVariableRef CVarResimRotStabilityMultiplier(TEXT("np2.Resim.RotStabilityMultiplier"), RotStabilityMultiplier, TEXT("Recommended range between 0.0-1.0. Lower value means more stable rotational corrections."));
	
		float VelStabilityMultiplier = 0.5f;
		static FAutoConsoleVariableRef CVarResimVelStabilityMultiplier(TEXT("np2.Resim.VelStabilityMultiplier"), VelStabilityMultiplier, TEXT("Recommended range between 0.0-1.0. Lower value means more stable linear velocity corrections."));

		float AngVelStabilityMultiplier = 0.5f;
		static FAutoConsoleVariableRef CVarResimAngVelStabilityMultiplier(TEXT("np2.Resim.AngVelStabilityMultiplier"), AngVelStabilityMultiplier, TEXT("Recommended range between 0.0-1.0. Lower value means more stable angular velocity corrections."));

		bool bDrawDebug = false;
		static FAutoConsoleVariableRef CVarResimDrawDebug(TEXT("np2.Resim.DrawDebug"), bDrawDebug, TEXT("Resimulation debug draw-calls"));

		float LogOutOfBoundsTimeLimit = 5.0f;
		static FAutoConsoleVariableRef CVarResimLogOutOfBoundsTimeLimit(TEXT("np2.Resim.LogOutOfBoundsTimeLimit"), LogOutOfBoundsTimeLimit, TEXT("Time that received targets needs to be within rewind bounds again before printing log that we are back in sync again. A new desync log will not be printed before an in-sync log has been printed."));
	}

	namespace PredictiveInterpolationCVars
	{
		float PosCorrectionTimeBase = 0.0f;
		static FAutoConsoleVariableRef CVarPosCorrectionTimeBase(TEXT("np2.PredictiveInterpolation.PosCorrectionTimeBase"), PosCorrectionTimeBase, TEXT("Base time to correct positional offset over. RoundTripTime * PosCorrectionTimeMultiplier is added on top of this."));

		float PosCorrectionTimeMin = 0.1f;
		static FAutoConsoleVariableRef CVarPosCorrectionTimeMin(TEXT("np2.PredictiveInterpolation.PosCorrectionTimeMin"), PosCorrectionTimeMin, TEXT("Min time to correct positional offset over. DeltaSeconds is added on top of this."));

		float PosCorrectionTimeMultiplier = 1.0f;
		static FAutoConsoleVariableRef CVarPosCorrectionTimeMultiplier(TEXT("np2.PredictiveInterpolation.PosCorrectionTimeMultiplier"), PosCorrectionTimeMultiplier, TEXT("Multiplier to adjust how much of RoundTripTime to add to positional offset correction."));

		float RotCorrectionTimeBase = 0.0f;
		static FAutoConsoleVariableRef CVarRotCorrectionTimeBase(TEXT("np2.PredictiveInterpolation.RotCorrectionTimeBase"), RotCorrectionTimeBase, TEXT("Base time to correct rotational offset over. RoundTripTime * RotCorrectionTimeMultiplier is added on top of this."));

		float RotCorrectionTimeMin = 0.1f;
		static FAutoConsoleVariableRef CVarRotCorrectionTimeMin(TEXT("np2.PredictiveInterpolation.RotCorrectionTimeMin"), RotCorrectionTimeMin, TEXT("Min time to correct rotational offset over. DeltaSeconds is added on top of this."));

		float RotCorrectionTimeMultiplier = 1.0f;
		static FAutoConsoleVariableRef CVarRotCorrectionTimeMultiplier(TEXT("np2.PredictiveInterpolation.RotCorrectionTimeMultiplier"), RotCorrectionTimeMultiplier, TEXT("Multiplier to adjust how much of RoundTripTime to add to rotational offset correction."));

		float PosInterpolationTimeMultiplier = 1.1f;
		static FAutoConsoleVariableRef CVarInterpolationTimeMultiplier(TEXT("np2.PredictiveInterpolation.InterpolationTimeMultiplier"), PosInterpolationTimeMultiplier, TEXT("Multiplier to adjust the interpolation time which is based on the sendrate of state data from the server."));
		
		float RotInterpolationTimeMultiplier = 1.25f;
		static FAutoConsoleVariableRef CVarRotInterpolationTimeMultiplier(TEXT("np2.PredictiveInterpolation.RotInterpolationTimeMultiplier"), RotInterpolationTimeMultiplier, TEXT("Multiplier to adjust the rotational interpolation time which is based on the sendrate of state data from the server."));
		
		float AverageReceiveIntervalSmoothing = 3.0f;
		static FAutoConsoleVariableRef CVarAverageReceiveIntervalSmoothing(TEXT("np2.PredictiveInterpolation.AverageReceiveIntervalSmoothing"), AverageReceiveIntervalSmoothing, TEXT("Recommended range: 1.0 - 5.0. Higher value makes the average receive interval adjust itself slower, reducing spikes in InterpolationTime."));

		float ExtrapolationTimeMultiplier = 3.0f;
		static FAutoConsoleVariableRef CVarExtrapolationTimeMultiplier(TEXT("np2.PredictiveInterpolation.ExtrapolationTimeMultiplier"), ExtrapolationTimeMultiplier, TEXT("Multiplier to adjust the time to extrapolate the target forward over, the time is based on current send-rate."));

		float ExtrapolationMinTime = 0.75f;
		static FAutoConsoleVariableRef CVarExtrapolationMinTime(TEXT("np2.PredictiveInterpolation.ExtrapolationMinTime"), ExtrapolationMinTime, TEXT("Clamps minimum extrapolation time. Value in seconds. Disable minimum clamp by setting to 0."));

		float MinExpectedDistanceCovered = 0.5f;
		static FAutoConsoleVariableRef CVarMinExpectedDistanceCovered(TEXT("np2.PredictiveInterpolation.MinExpectedDistanceCovered"), MinExpectedDistanceCovered, TEXT("Value between 0-1, in percentage where 0.25 = 25%. How much of the expected distance based on replication velocity should the object have covered in a simulation tick to Not be considered stuck."));

		float ErrorAccumulationDecreaseMultiplier = 0.5f;
		static FAutoConsoleVariableRef CVarErrorAccumulationDecreaseMultiplier(TEXT("np2.PredictiveInterpolation.ErrorAccumulationDecreaseMultiplier"), ErrorAccumulationDecreaseMultiplier, TEXT("Multiplier to adjust how fast we decrease accumulated error time when we no longer accumulate error."));

		float ErrorAccumulationSeconds = 3.0f;
		static FAutoConsoleVariableRef CVarErrorAccumulationSeconds(TEXT("np2.PredictiveInterpolation.ErrorAccumulationSeconds"), ErrorAccumulationSeconds, TEXT("Perform a reposition if replication have not been able to cover the min expected distance towards the target for this amount of time."));
		
		bool bDisableErrorVelocityLimits = false;
		static FAutoConsoleVariableRef CVarDisableErrorVelocityLimits(TEXT("np2.PredictiveInterpolation.DisableErrorVelocityLimits"), bDisableErrorVelocityLimits, TEXT("Disable the velocity limit and allow error accumulation at any velocity."));
		
		float ErrorAccLinVelMaxLimit = 50.0f;
		static FAutoConsoleVariableRef CVarErrorAccLinVelMaxLimit(TEXT("np2.PredictiveInterpolation.ErrorAccLinVelMaxLimit"), ErrorAccLinVelMaxLimit, TEXT("If target velocity is below this limit we check for desync to trigger softsnap and accumulate time to build up to a hardsnap."));
		
		float ErrorAccAngVelMaxLimit = 1.5f;
		static FAutoConsoleVariableRef CVarErrorAccAngVelMaxLimit(TEXT("np2.PredictiveInterpolation.ErrorAccAngVelMaxLimit"), ErrorAccAngVelMaxLimit, TEXT("If target angular velocity (in radians) is below this limit we check for desync to trigger softsnap and accumulate time to build up to a hardsnap."));
		
		float SoftSnapPosStrength = 0.5f;
		static FAutoConsoleVariableRef CVarSoftSnapPosStrength(TEXT("np2.PredictiveInterpolation.SoftSnapPosStrength"), SoftSnapPosStrength, TEXT("Value in percent between 0.0 - 1.0 representing how much to softsnap each tick of the remaining positional distance."));
		
		float SoftSnapRotStrength = 0.5f;
		static FAutoConsoleVariableRef CVarSoftSnapRotStrength(TEXT("np2.PredictiveInterpolation.SoftSnapRotStrength"), SoftSnapRotStrength, TEXT("Value in percent between 0.0 - 1.0 representing how much to softsnap each tick of the remaining rotational distance."));

		bool bSoftSnapToSource = false;
		static FAutoConsoleVariableRef CVarSoftSnapToSource(TEXT("np2.PredictiveInterpolation.SoftSnapToSource"), bSoftSnapToSource, TEXT("If true, soft snap will be performed towards the source state of the current target instead of the predicted state of the current target."));

		float EarlyOutDistanceSqr = 1.0f;
		static FAutoConsoleVariableRef CVarEarlyOutDistanceSqr(TEXT("np2.PredictiveInterpolation.EarlyOutDistanceSqr"), EarlyOutDistanceSqr, TEXT("Squared value. If object is within this distance from the source target, early out from replication and apply sleep if replicated."));
		
		float EarlyOutAngle = 1.5f;
		static FAutoConsoleVariableRef CVarEarlyOutAngle(TEXT("np2.PredictiveInterpolation.EarlyOutAngle"), EarlyOutAngle, TEXT("If object is within this rotational angle (in degrees) from the source target, early out from replication and apply sleep if replicated."));
		
		bool bEarlyOutWithVelocity = true;
		static FAutoConsoleVariableRef CVarEarlyOutWithVelocity(TEXT("np2.PredictiveInterpolation.EarlyOutWithVelocity"), bEarlyOutWithVelocity, TEXT("If true, allow replication logic to early out if current velocities are driving replication well enough. If false, only early out if target velocity is zero."));

		bool bSkipVelocityRepOnPosEarlyOut = true;
		static FAutoConsoleVariableRef CVarSkipVelocityRepOnPosEarlyOut(TEXT("np2.PredictiveInterpolation.SkipVelocityRepOnPosEarlyOut"), bSkipVelocityRepOnPosEarlyOut, TEXT("If true, don't run linear velocity replication if position can early out but angular can't early out."));

		bool bPostResimWaitForUpdate = false;
		static FAutoConsoleVariableRef CVarPostResimWaitForUpdate(TEXT("np2.PredictiveInterpolation.PostResimWaitForUpdate"), bPostResimWaitForUpdate, TEXT("After a resimulation, wait for replicated states that correspond to post-resim state before processing replication again."));
		
		bool bVelocityBased = true;
		static FAutoConsoleVariableRef CVarVelocityBased(TEXT("np2.PredictiveInterpolation.VelocityBased"), bVelocityBased, TEXT("When true, predictive interpolation replication mode will move objects replicate by applying linear velocity and angular velocity. When false, objects will move by transform shifting towards the target state."));
		
		bool bCorrectionAsVelocity = false;
		static FAutoConsoleVariableRef CVarCorrectionAsVelocity(TEXT("np2.PredictiveInterpolation.CorrectionAsVelocity"), bCorrectionAsVelocity, TEXT("When true, predictive interpolation will apply positional and rotational offset correction as a velocity instead of as a transform shift."));

		bool bCorrectConnectedBodies = false;
		static FAutoConsoleVariableRef CVarCorrectConnectedBodies(TEXT("np2.PredictiveInterpolation.CorrectConnectedBodies"), bCorrectConnectedBodies, TEXT("When true, transform corrections will also apply to any connected physics object."));

		bool bCorrectConnectedBodiesFriction = true;
		static FAutoConsoleVariableRef CVarCorrectConnectedBodiesFriction(TEXT("np2.PredictiveInterpolation.CorrectConnectedBodiesFriction"), bCorrectConnectedBodiesFriction, TEXT("When true, transform correction on any connected physics object will also recalculate their friction."));
		
		bool bSleepConnectedBodies = true;
		static FAutoConsoleVariableRef CVarSleepConnectedBodies(TEXT("np2.PredictiveInterpolation.SleepConnectedBodies"), bSleepConnectedBodies, TEXT("When true, sleep state will be applied to any dynamic physics object connected to the replicated object."));

		bool bKinematicPrediction = true;
		static FAutoConsoleVariableRef CVarKinematicPrediction(TEXT("np2.PredictiveInterpolation.KinematicPrediction"), bKinematicPrediction, TEXT("When true, predictive interpolation will perform predictive movement instead of interpolation for kinematic objects."));
	
		bool bKinematicHardSnap = false;
		static FAutoConsoleVariableRef CVarKinematicHardSnap(TEXT("np2.PredictiveInterpolation.KinematicHardSnap"), bKinematicHardSnap, TEXT("When true, predictive interpolation will perform a hard snap for objects that are kinematic."));
	
		bool bDisableSoftSnap = false;
		static FAutoConsoleVariableRef CVarDisableSoftSnap(TEXT("np2.PredictiveInterpolation.DisableSoftSnap"), bDisableSoftSnap, TEXT("When true, predictive interpolation will not use softsnap to correct the replication with when velocity fails. Hardsnap will still eventually kick in if replication can't reach the target."));

		bool bAlwaysHardSnap = false;
		static FAutoConsoleVariableRef CVarAlwaysHardSnap(TEXT("np2.PredictiveInterpolation.AlwaysHardSnap"), bAlwaysHardSnap, TEXT("When true, predictive interpolation replication mode will always hard snap. Used as a backup measure"));

		bool bSkipReplication = false;
		static FAutoConsoleVariableRef CVarSkipReplication(TEXT("np2.PredictiveInterpolation.SkipReplication"), bSkipReplication, TEXT("When true, predictive interpolation is not applied anymore letting the object simulate freely instead"));

		bool bDontClearTarget = false;
		static FAutoConsoleVariableRef CVarDontClearTarget(TEXT("np2.PredictiveInterpolation.DontClearTarget"), bDontClearTarget, TEXT("When true, predictive interpolation will not lose track of the last replicated state after coming to rest."));
		
		bool bDrawDebugTargets = false;
		static FAutoConsoleVariableRef CVarDrawDebugTargets(TEXT("np2.PredictiveInterpolation.DrawDebugTargets"), bDrawDebugTargets, TEXT("Draw target states, color coded by which ServerFrame they originate from, replicated targets are large and extrapolated targets are small. There is a Z offset to the draw calls."));
		
		bool bDrawDebugVectors = false;
		static FAutoConsoleVariableRef CVarDrawDebugVectors(TEXT("np2.PredictiveInterpolation.DrawDebugVectors"), bDrawDebugVectors, TEXT("Draw replication vectors, target velocity, replicated velocity, velocity change between replication calls etc."));
		
		float DrawDebugZOffset = 50.0f;
		static FAutoConsoleVariableRef CVarDrawDebugZOffset(TEXT("np2.PredictiveInterpolation.DrawDebugZOffset"), DrawDebugZOffset, TEXT("Offset in Z axis for draw debug calls"));

		float SleepSecondsClearTarget = 15.0f;
		static FAutoConsoleVariableRef CVarSleepSecondsClearTarget(TEXT("np2.PredictiveInterpolation.SleepSecondsClearTarget"), SleepSecondsClearTarget, TEXT("Wait for the object to sleep for this many seconds before clearing the replication target, to ensure nothing wakes up the object just after it goes to sleep on the client."));
		
		int32 TargetTickAlignmentClampMultiplier = 2;
		static FAutoConsoleVariableRef CVarTargetTickAlignmentClampMultiplier(TEXT("np2.PredictiveInterpolation.TargetTickAlignmentClampMultiplier"), TargetTickAlignmentClampMultiplier, TEXT("Multiplier to adjust clamping of target alignment via TickCount. Multiplier is performed on AverageReceiveInterval."));

		int32 TeleportDetectionEnabled = 1;
		static FAutoConsoleVariableRef CVarTeleportDetectionEnabled(TEXT("np2.PredictiveInterpolation.TeleportDetection.Enabled"), TeleportDetectionEnabled, TEXT("Set to 1 to enable teleportation detection which hard snaps the replicated object if replication thinks a teleportation has happened. Disable by setting to 0 (or any other value currently, leaving room for adding more teleportation detection modes which will use incremental values)."));

		float TeleportDetectionMinDistance = 200.0f;
		static FAutoConsoleVariableRef CVarTeleportDetectionMinDistance(TEXT("np2.PredictiveInterpolation.TeleportDetection.MinDistance"), TeleportDetectionMinDistance, TEXT("Minimum positional distance between two received target states running teleportation detection."));

		float TeleportDetectionVelocityMultiplier = 1.3f;
		static FAutoConsoleVariableRef CVarTeleportDetectionVelocityMultiplier(TEXT("np2.PredictiveInterpolation.TeleportDetection.VelocityMultiplier"), TeleportDetectionVelocityMultiplier, TEXT("Multiplier to add leniency when checking if the previous or current velocity could cover the positional offset between previous and current target state. Higher value results in more lenient velocity comparison, i.e. less risk of triggering a hard snap when a teleport has not happened but also less likely to catch a teleport for objects that move while being teleported. Values under 1.0 are not recommended."));
	}

}

namespace Chaos
{
	extern CHAOS_API int32 RewindBeforeAdvance;
}


FPhysicsReplication::FPhysicsReplication(FPhysScene* InPhysicsScene)
	: PhysScene(InPhysicsScene)
{
	using namespace Chaos;
	AsyncInput = nullptr;
	PhysicsReplicationAsync = nullptr;
	if (auto* Solver = PhysScene->GetSolver())
	{
		PhysicsReplicationAsync = Solver->CreateAndRegisterSimCallbackObject_External<FPhysicsReplicationAsync>();
		PhysicsReplicationAsync->Setup(UPhysicsSettings::Get()->PhysicErrorCorrection);
	}
}

FPhysicsReplication::~FPhysicsReplication()
{
	if (PhysicsReplicationAsync)
	{
		if (auto* Solver = PhysScene->GetSolver())
		{
			Solver->UnregisterAndFreeSimCallbackObject_External(PhysicsReplicationAsync);
		}
	}
}


void FPhysicsReplication::SetReplicatedTarget(UPrimitiveComponent* Component, FName BoneName, const FRigidBodyState& ReplicatedTarget, int32 ServerFrame)
{
	// If networked physics prediction is enabled, enforce the new physics replication flow via SetReplicatedTarget() using PhysicsObject instead of BodyInstance from BoneName.
	AActor* Owner = Component->GetOwner();
	if (Owner && (PhysicsReplicationCVars::EnableDefaultReplication || Owner->GetPhysicsReplicationMode() != EPhysicsReplicationMode::Default)) // For now, only opt in to the PhysicsObject flow if not using Default replication or if default is allowed via CVar.
	{
		const ENetRole OwnerRole = Owner->GetLocalRole();
		const bool bIsSimulated = OwnerRole == ROLE_SimulatedProxy;
		const bool bIsReplicatedAutonomous = OwnerRole == ROLE_AutonomousProxy && Component->bReplicatePhysicsToAutonomousProxy;
		if (bIsSimulated || bIsReplicatedAutonomous)
		{
			Chaos::FConstPhysicsObjectHandle PhysicsObject = Component->GetPhysicsObjectByName(BoneName);
			SetReplicatedTarget(PhysicsObject, ReplicatedTarget, ServerFrame, Owner->GetPhysicsReplicationMode());
			return;
		}
	}

	if (UWorld* OwningWorld = GetOwningWorld())
	{
		//TODO: there's a faster way to compare this
		TWeakObjectPtr<UPrimitiveComponent> TargetKey(Component);
		FReplicatedPhysicsTarget* Target = ComponentToTargets_DEPRECATED.Find(TargetKey);
		if (!Target)
		{
			// First time we add a target, set it's previous and correction
			// positions to the target position to avoid math with uninitialized
			// memory.
			Target = &ComponentToTargets_DEPRECATED.Add(TargetKey);
			Target->PrevPos = ReplicatedTarget.Position;
			Target->PrevPosTarget = ReplicatedTarget.Position;
		}

		Target->ServerFrame = ServerFrame;
		Target->TargetState = ReplicatedTarget;
		Target->BoneName = BoneName;
		Target->ArrivedTimeSeconds = OwningWorld->GetTimeSeconds();

		ensure(!Target->PrevPos.ContainsNaN());
		ensure(!Target->PrevPosTarget.ContainsNaN());
		ensure(!Target->TargetState.Position.ContainsNaN());

		OnSetReplicatedTarget(Component, BoneName, ReplicatedTarget, ServerFrame, *Target);
	}
}

void FPhysicsReplication::SetReplicatedTarget(Chaos::FConstPhysicsObjectHandle PhysicsObject, const FRigidBodyState& ReplicatedTarget, int32 ServerFrame, EPhysicsReplicationMode ReplicationMode)
{
	if (!PhysicsObject)
	{
		return;
	}

	UWorld* OwningWorld = GetOwningWorld();
	if (OwningWorld == nullptr)
	{
		return;
	}

	// TODO, Check if owning actor is ROLE_SimulatedProxy or ROLE_AutonomousProxy ?

	FReplicatedPhysicsTarget Target(PhysicsObject);
	Target.ReplicationMode = ReplicationMode;
	Target.ServerFrame = ServerFrame;
	Target.TargetState = ReplicatedTarget;
	Target.ArrivedTimeSeconds = OwningWorld->GetTimeSeconds();

	ensure(!Target.TargetState.Position.ContainsNaN());

	ReplicatedTargetsQueue.Add(Target);
}

void FPhysicsReplication::RemoveReplicatedTarget(UPrimitiveComponent* Component)
{
	if (Component == nullptr)
	{
		return;
	}

	// Remove from legacy flow
	ComponentToTargets_DEPRECATED.Remove(Component);
	
	// Remove from FPhysicsObject flow
	Chaos::FConstPhysicsObjectHandle PhysicsObject = Component->GetPhysicsObjectByName(NAME_None);
	RemoveReplicatedTarget(PhysicsObject);
}
void FPhysicsReplication::RemoveReplicatedTarget(Chaos::FConstPhysicsObjectHandle PhysicsObject)
{
	// Remove from FPhysicsObject flow
	if (!PhysicsObject)
	{
		return;
	}

	FReplicatedPhysicsTarget Target(PhysicsObject); // This creates a new but empty target and when it tries to update the current target in the async flow it will remove it from replication since it's empty.
	ReplicatedTargetsQueue.Add(Target);
}


void FPhysicsReplication::Tick(float DeltaSeconds)
{
	OnTick(DeltaSeconds, ComponentToTargets_DEPRECATED);
}

void FPhysicsReplication::OnTick(float DeltaSeconds, TMap<TWeakObjectPtr<UPrimitiveComponent>, FReplicatedPhysicsTarget>& ComponentsToTargets)
{
	using namespace Chaos;

	if (ShouldSkipPhysicsReplication())
	{
		return;
	}

	// Don't tick unless we have data to process
	if (ComponentsToTargets.Num() == 0 && ReplicatedTargetsQueue.Num() == 0)
	{
		return;
	}

	NetworkPhysicsTickOffsetAssigned = false;
	NetworkPhysicsTickOffset = 0; // LocalFrame = ServerFrame - NetworkPhysicsTickOffset;

	if (UPhysicsSettings::Get()->PhysicsPrediction.bEnablePhysicsPrediction)
	{
		if (UWorld* World = GetOwningWorld())
		{
			if (APlayerController* PlayerController = World->GetFirstPlayerController())
			{
				NetworkPhysicsTickOffsetAssigned = PlayerController->GetNetworkPhysicsTickOffsetAssigned();
				NetworkPhysicsTickOffset = PlayerController->GetNetworkPhysicsTickOffset();
			}
		}
	}

	const FRigidBodyErrorCorrection& PhysicErrorCorrection = UPhysicsSettings::Get()->PhysicErrorCorrection;
	if (PhysicsReplicationAsync)
	{
		PrepareAsyncData_External(PhysicErrorCorrection);
		if (ensure(AsyncInput))
		{
			AsyncInput->NetworkPhysicsTickOffsetAssigned = NetworkPhysicsTickOffsetAssigned;
			AsyncInput->NetworkPhysicsTickOffset = NetworkPhysicsTickOffset;
		}
	}

	// Get the ping between this PC & the server
	const float LocalPing = GetLocalPing();

	// BodyInstance replication flow, deprecated
	for (auto Itr = ComponentsToTargets.CreateIterator(); Itr; ++Itr)
	{
		bool bRemoveItr = false;
		if (UPrimitiveComponent* PrimComp = Itr.Key().Get())
		{
			if (PrimComp->GetAttachParent() == nullptr)
			{
				if (FBodyInstance* BI = PrimComp->GetBodyInstance(Itr.Value().BoneName))
				{
					FReplicatedPhysicsTarget& PhysicsTarget = Itr.Value();
					FRigidBodyState& UpdatedState = PhysicsTarget.TargetState;
					bool bUpdated = false;
					if (AActor* OwningActor = PrimComp->GetOwner())
					{
						// Update actor replication settings overrides
						SettingsCurrent = UNetworkPhysicsSettingsComponent::GetSettingsForActor(OwningActor);

						const ENetRole OwnerRole = OwningActor->GetLocalRole();
						const bool bIsSimulated = OwnerRole == ROLE_SimulatedProxy;
						const bool bIsReplicatedAutonomous = OwnerRole == ROLE_AutonomousProxy && PrimComp->bReplicatePhysicsToAutonomousProxy;
						if (bIsSimulated || bIsReplicatedAutonomous)
						{
							// Get the ping of this thing's owner. If nobody owns it,
							// then it's server authoritative.
							const float OwnerPing = GetOwnerPing(OwningActor, PhysicsTarget);

							// Get the total ping - this approximates the time since the update was
							// actually generated on the machine that is doing the authoritative sim.
							// NOTE: We divide by 2 to approximate 1-way ping from 2-way ping.
							const float PingSecondsOneWay = (LocalPing + OwnerPing) * 0.5f * 0.001f;

							if (UpdatedState.Flags & ERigidBodyFlags::NeedsUpdate)
							{
								const int32 LocalFrame = PhysicsTarget.ServerFrame - NetworkPhysicsTickOffset;
								const bool bRestoredState = ApplyRigidBodyState(DeltaSeconds, BI, PhysicsTarget, PhysicErrorCorrection, PingSecondsOneWay, LocalFrame, 0);

								// Need to update the component to match new position.
								if (PhysicsReplicationCVars::SkipSkeletalRepOptimization == 0 || Cast<USkeletalMeshComponent>(PrimComp) == nullptr)	//simulated skeletal mesh does its own polling of physics results so we don't need to call this as it'll happen at the end of the physics sim
								{
									PrimComp->SyncComponentToRBPhysics();
								}
								if (bRestoredState)
								{
									bRemoveItr = true;
								}
							}
						}
					}
				}
			}
		}

		if (bRemoveItr)
		{
			OnTargetRestored(Itr.Key().Get(), Itr.Value());
			PendingDeleteFromComponentsToTargets.Add(Itr.Key());
		}
	}

	for (TWeakObjectPtr<UPrimitiveComponent>& PrimitiveComponent : PendingDeleteFromComponentsToTargets)
	{
		ComponentsToTargets.Remove(PrimitiveComponent);
	}
	PendingDeleteFromComponentsToTargets.Reset();

	if (AsyncInput)
	{
		// PhysicsObject replication flow
		for (FReplicatedPhysicsTarget& PhysicsTarget : ReplicatedTargetsQueue)
		{
			const float PingSecondsOneWay = LocalPing * 0.5f * 0.001f;

			// Queue up the target state for async replication
			FPhysicsRepAsyncInputData AsyncInputData(PhysicsTarget.PhysicsObject);
			AsyncInputData.TargetState = PhysicsTarget.TargetState;
			AsyncInputData.Proxy = nullptr;
			AsyncInputData.RepMode = PhysicsTarget.ReplicationMode;
			AsyncInputData.ServerFrame = PhysicsTarget.ServerFrame;
			AsyncInputData.LatencyOneWay = PingSecondsOneWay;
		
			AsyncInput->InputData.Add(AsyncInputData);
		}
	}
	ReplicatedTargetsQueue.Reset();

	AsyncInput = nullptr;
}

namespace
{

	// Helper to return the deltas between current and target Position and Rotation
	void ComputeDeltas(const FVector& CurrentPos, const FQuat& CurrentQuat, const FVector& TargetPos, const FQuat& TargetQuat, FVector& OutLinDiff, float& OutLinDiffSize,
		FVector& OutAngDiffAxis, float& OutAngDiff, float& OutAngDiffSize)
	{
		OutLinDiff = TargetPos - CurrentPos;
		OutLinDiffSize = OutLinDiff.Size();
		const FQuat InvCurrentQuat = CurrentQuat.Inverse();
		const FQuat DeltaQuat = TargetQuat * InvCurrentQuat;
		DeltaQuat.ToAxisAndAngle(OutAngDiffAxis, OutAngDiff);
		OutAngDiff = FMath::RadiansToDegrees(FMath::UnwindRadians(OutAngDiff));
		OutAngDiffSize = FMath::Abs(OutAngDiff);
	}
}

bool FPhysicsReplication::ApplyRigidBodyState(float DeltaSeconds, FBodyInstance* BI, FReplicatedPhysicsTarget& PhysicsTarget, const FRigidBodyErrorCorrection& ErrorCorrection, const float InPingSecondsOneWay, int32 LocalFrame, int32 NumPredictedFrames)
{
	// Call into the old ApplyRigidBodyState function for now,
	// Note that old ApplyRigidBodyState is overridden in other projects, so consider backwards compatible path
	return ApplyRigidBodyState(DeltaSeconds, BI, PhysicsTarget, ErrorCorrection, InPingSecondsOneWay, nullptr);
}

bool FPhysicsReplication::ApplyRigidBodyState(float DeltaSeconds, FBodyInstance* BI, FReplicatedPhysicsTarget& PhysicsTarget, const FRigidBodyErrorCorrection& ErrorCorrection,
	const float PingSecondsOneWay, bool* bDidHardSnap)
{
	if (!BI->IsInstanceSimulatingPhysics())
	{
		return false;
	}

	//
	// NOTES:
	//
	// The operation of this method has changed since 4.18.
	//
	// When a new remote physics state is received, this method will
	// be called on tick until the local state is within an adequate
	// tolerance of the new state.
	//
	// The received state is extrapolated based on ping, by some
	// adjustable amount.
	//
	// A correction velocity is added new state's velocity, and assigned
	// to the body. The correction velocity scales with the positional
	// difference, so without the interference of external forces, this
	// will result in an exponentially decaying correction.
	//
	// Generally it is not needed and will interrupt smoothness of
	// the replication, but stronger corrections can be obtained by
	// adjusting position lerping.
	//
	// If progress is not being made towards equilibrium, due to some
	// divergence in physics states between the owning and local sims,
	// an error value is accumulated, representing the amount of time
	// spent in an unresolvable state.
	//
	// Once the error value has exceeded some threshold (0.5 seconds
	// by default), a hard snap to the target physics state is applied.
	//

	bool bRestoredState = true;
	const FRigidBodyState NewState = PhysicsTarget.TargetState;
	const float NewQuatSizeSqr = NewState.Quaternion.SizeSquared();

	// failure cases
	if (!BI->IsInstanceSimulatingPhysics())
	{
		UE_LOGF(LogPhysics, Warning, "Physics replicating on non-simulated body. (%ls)", *BI->GetBodyDebugName());
		return bRestoredState;
	}
	else if (NewQuatSizeSqr < UE_KINDA_SMALL_NUMBER)
	{
		UE_LOGF(LogPhysics, Warning, "Invalid zero quaternion set for body. (%ls)", *BI->GetBodyDebugName());
		return bRestoredState;
	}
	else if (FMath::Abs(NewQuatSizeSqr - 1.f) > UE_KINDA_SMALL_NUMBER)
	{
		UE_LOGF(LogPhysics, Warning, "Quaternion (%f %f %f %f) with non-unit magnitude detected. (%ls)",
			NewState.Quaternion.X, NewState.Quaternion.Y, NewState.Quaternion.Z, NewState.Quaternion.W, *BI->GetBodyDebugName());
		return bRestoredState;
	}

	// Grab configuration variables from engine config or from CVars if overriding is turned on.
	const float NetPingExtrapolation = CharacterMovementCVars::NetPingExtrapolation >= 0.0f ? CharacterMovementCVars::NetPingExtrapolation : ErrorCorrection.PingExtrapolation;
	const float NetPingLimit = CharacterMovementCVars::NetPingLimit > 0.0f ? CharacterMovementCVars::NetPingLimit : ErrorCorrection.PingLimit;
	const float ErrorPerLinearDiff = CharacterMovementCVars::ErrorPerLinearDifference >= 0.0f ? CharacterMovementCVars::ErrorPerLinearDifference : ErrorCorrection.ErrorPerLinearDifference;
	const float ErrorPerAngularDiff = CharacterMovementCVars::ErrorPerAngularDifference >= 0.0f ? CharacterMovementCVars::ErrorPerAngularDifference : ErrorCorrection.ErrorPerAngularDifference;
	const float MaxRestoredStateError = CharacterMovementCVars::MaxRestoredStateError >= 0.0f ? CharacterMovementCVars::MaxRestoredStateError : ErrorCorrection.MaxRestoredStateError;
	const float ErrorAccumulationSeconds = CharacterMovementCVars::ErrorAccumulationSeconds >= 0.0f ? CharacterMovementCVars::ErrorAccumulationSeconds : ErrorCorrection.ErrorAccumulationSeconds;
	const float ErrorAccumulationDistanceSq = CharacterMovementCVars::ErrorAccumulationDistanceSq >= 0.0f ? CharacterMovementCVars::ErrorAccumulationDistanceSq : ErrorCorrection.ErrorAccumulationDistanceSq;
	const float ErrorAccumulationSimilarity = CharacterMovementCVars::ErrorAccumulationSimilarity >= 0.0f ? CharacterMovementCVars::ErrorAccumulationSimilarity : ErrorCorrection.ErrorAccumulationSimilarity;
	const float PositionLerp = CharacterMovementCVars::PositionLerp >= 0.0f ? CharacterMovementCVars::PositionLerp : ErrorCorrection.PositionLerp;
	const float LinearVelocityCoefficient = CharacterMovementCVars::LinearVelocityCoefficient >= 0.0f ? CharacterMovementCVars::LinearVelocityCoefficient : ErrorCorrection.LinearVelocityCoefficient;
	const float AngleLerp = CharacterMovementCVars::AngleLerp >= 0.0f ? CharacterMovementCVars::AngleLerp : ErrorCorrection.AngleLerp;
	const float AngularVelocityCoefficient = CharacterMovementCVars::AngularVelocityCoefficient >= 0.0f ? CharacterMovementCVars::AngularVelocityCoefficient : ErrorCorrection.AngularVelocityCoefficient;
	float MaxLinearHardSnapDistance = CharacterMovementCVars::MaxLinearHardSnapDistance >= 0.f ? CharacterMovementCVars::MaxLinearHardSnapDistance : ErrorCorrection.MaxLinearHardSnapDistance;

	bool bHardsnapLegacyInPT = PhysicsReplicationCVars::DefaultReplicationCVars::bHardsnapLegacyInPT;
	bool bCorrectConnectedBodies = PhysicsReplicationCVars::DefaultReplicationCVars::bCorrectConnectedBodies;
	bool bCorrectConnectedBodiesFriction = PhysicsReplicationCVars::DefaultReplicationCVars::bCorrectConnectedBodiesFriction;

	// Assign per-actor settings from NetworkPhysicSettingsComponent if this actor has one
	if (SettingsCurrent.IsValid())
	{
		const FNetworkPhysicsSettingsData& SettingsData = SettingsCurrent.Pin()->GetSettings();
		MaxLinearHardSnapDistance = SettingsData.DefaultReplicationSettings.GetMaxLinearHardSnapDistance(MaxLinearHardSnapDistance);
		bHardsnapLegacyInPT = SettingsData.DefaultReplicationSettings.GetHardsnapDefaultLegacyInPT();
		bCorrectConnectedBodies = SettingsData.DefaultReplicationSettings.GetCorrectConnectedBodies();
		bCorrectConnectedBodiesFriction = SettingsData.DefaultReplicationSettings.GetCorrectConnectedBodiesFriction();
	}

	// Get Current state
	FRigidBodyState CurrentState;
	BI->GetRigidBodyState(CurrentState);

	/////// EXTRAPOLATE APPROXIMATE TARGET VALUES ///////

	// Starting from the last known authoritative position, and
	// extrapolate an approximation using the last known velocity
	// and ping.
	const float PingSeconds = FMath::Clamp(PingSecondsOneWay, 0.f, NetPingLimit);
	const float ExtrapolationDeltaSeconds = PingSeconds * NetPingExtrapolation;
	const FVector ExtrapolationDeltaPos = NewState.LinVel * ExtrapolationDeltaSeconds;
	const FVector_NetQuantize100 TargetPos = NewState.Position + ExtrapolationDeltaPos;
	float NewStateAngVel;
	FVector NewStateAngVelAxis;
	NewState.AngVel.FVector::ToDirectionAndLength(NewStateAngVelAxis, NewStateAngVel);
	NewStateAngVel = FMath::DegreesToRadians(NewStateAngVel);
	const FQuat ExtrapolationDeltaQuaternion = FQuat(NewStateAngVelAxis, NewStateAngVel * ExtrapolationDeltaSeconds);
	FQuat TargetQuat = ExtrapolationDeltaQuaternion * NewState.Quaternion;

	/////// COMPUTE DIFFERENCES ///////
	FVector LinDiff;
	float LinDiffSize;
	FVector AngDiffAxis;
	float AngDiff;
	float AngDiffSize;

	ComputeDeltas(CurrentState.Position, CurrentState.Quaternion, TargetPos, TargetQuat, LinDiff, LinDiffSize, AngDiffAxis, AngDiff, AngDiffSize);

	/////// ACCUMULATE ERROR IF NOT APPROACHING SOLUTION ///////

	// Store sleeping state
	const bool bShouldSleep = (NewState.Flags & ERigidBodyFlags::Sleeping) != 0;
	const bool bWasAwake = BI->IsInstanceAwake();
	const bool bAutoWake = false;

	const float Error = (LinDiffSize * ErrorPerLinearDiff) + (AngDiffSize * ErrorPerAngularDiff);
	bRestoredState = Error < MaxRestoredStateError;
	if (bRestoredState)
	{
		PhysicsTarget.AccumulatedErrorSeconds = 0.0f;
	}
	else
	{
		//
		// The heuristic for error accumulation here is:
		// 1. Did the physics tick from the previous step fail to
		//    move the body towards a resolved position?
		// 2. Was the linear error in the same direction as the
		//    previous frame?
		// 3. Is the linear error large enough to accumulate error?
		//
		// If these conditions are met, then "error" time will accumulate.
		// Once error has accumulated for a certain number of seconds,
		// a hard-snap to the target will be performed.
		//
		// TODO: Rotation while moving linearly can still mess up this
		// heuristic. We need to account for it.
		//

		// Project the change in position from the previous tick onto the
		// linear error from the previous tick. This value roughly represents
		// how much correction was performed over the previous physics tick.
		const float PrevProgress = FVector::DotProduct(
			FVector(CurrentState.Position) - PhysicsTarget.PrevPos,
			(PhysicsTarget.PrevPosTarget - PhysicsTarget.PrevPos).GetSafeNormal());

		// Project the current linear error onto the linear error from the
		// previous tick. This value roughly represents how little the direction
		// of the linear error state has changed, and how big the error is.
		const float PrevSimilarity = FVector::DotProduct(
			TargetPos - FVector(CurrentState.Position),
			PhysicsTarget.PrevPosTarget - PhysicsTarget.PrevPos);

		// If the conditions from the heuristic outlined above are met, accumulate
		// error. Otherwise, reduce it.
		if (PrevProgress < ErrorAccumulationDistanceSq &&
			PrevSimilarity > ErrorAccumulationSimilarity)
		{
			PhysicsTarget.AccumulatedErrorSeconds += DeltaSeconds;
		}
		else
		{
			PhysicsTarget.AccumulatedErrorSeconds = FMath::Max(PhysicsTarget.AccumulatedErrorSeconds - DeltaSeconds, 0.0f);
		}

		// Hard snap if error accumulation or linear error is big enough, and clear the error accumulator.
		const bool bHardSnap =
			LinDiffSize > MaxLinearHardSnapDistance ||
			PhysicsTarget.AccumulatedErrorSeconds > ErrorAccumulationSeconds ||
			CharacterMovementCVars::AlwaysHardSnap;

		const FTransform IdealWorldTM(TargetQuat, TargetPos);

		if (bHardSnap)
		{
#if !UE_BUILD_SHIPPING
			if (PhysicsReplicationCVars::LogPhysicsReplicationHardSnaps && GetOwningWorld())
			{
				UE_LOGF(LogTemp, Warning, "Simulated HARD SNAP - \nCurrent Pos - %ls, Target Pos - %ls\n CurrentState.LinVel - %ls, New Lin Vel - %ls\nTarget Extrapolation Delta - %ls, Is Replay? - %d, Is Asleep - %d, Prev Progress - %f, Prev Similarity - %f",
					*CurrentState.Position.ToString(), *TargetPos.ToString(), *CurrentState.LinVel.ToString(), *NewState.LinVel.ToString(),
					*ExtrapolationDeltaPos.ToString(), GetOwningWorld()->IsPlayingReplay(), !BI->IsInstanceAwake(), PrevProgress, PrevSimilarity);
				if (bDidHardSnap)
				{
					*bDidHardSnap = true;
				}
				if (LinDiffSize > MaxLinearHardSnapDistance)
				{
					UE_LOGF(LogTemp, Warning, "Hard snap due to linear difference error");
				}
				else
				{
					UE_LOGF(LogTemp, Warning, "Hard snap due to accumulated error")
				}
			}
#endif
			// Too much error so just snap state here and be done with it
			PhysicsTarget.AccumulatedErrorSeconds = 0.0f;
			bRestoredState = true;

			// Hardsnap in physics thread
			bool bPTHardSnapSuccess = false;
			if (PhysicsReplicationAsync != nullptr)
			{
				if (bHardsnapLegacyInPT)
				{
					if (Chaos::FSingleParticlePhysicsProxy* Proxy = static_cast<Chaos::FSingleParticlePhysicsProxy*>(BI->GetPhysicsActor()))
					{
						if (Chaos::FPBDRigidsSolver* Solver = Proxy->GetSolver<Chaos::FPBDRigidsSolver>())
						{
							Solver->EnqueueCommandImmediate([Solver, Proxy, IdealWorldTM, NewState, bCorrectConnectedBodies, bCorrectConnectedBodiesFriction]()
								{
									Chaos::FRigidBodyHandle_Internal* Handle = Proxy->GetPhysicsThreadAPI();
								
									// Set XRVW to hard snap dynamic object and force recalculation of friction
									Solver->GetEvolution()->ApplyParticleTransformCorrection(Proxy->GetHandle_LowLevel(), IdealWorldTM.GetLocation(), IdealWorldTM.GetRotation(), bCorrectConnectedBodies, bCorrectConnectedBodiesFriction);

									Handle->SetV(NewState.LinVel);
									Handle->SetW(FMath::DegreesToRadians(NewState.AngVel));
								});

							bPTHardSnapSuccess = true;
						}
					}
				}
			}

			if (!bPTHardSnapSuccess)
			{
				BI->SetBodyTransform(IdealWorldTM, ETeleportType::ResetPhysics, bAutoWake);

				// Set the new velocities
				BI->SetLinearVelocity(NewState.LinVel, false, bAutoWake);
				BI->SetAngularVelocityInRadians(FMath::DegreesToRadians(NewState.AngVel), false, bAutoWake);
			}
		}
		else
		{
			// Small enough error to interpolate
			if (PhysicsReplicationAsync == nullptr)	//sync case
			{
				const FVector NewLinVel = FVector(NewState.LinVel) + (LinDiff * LinearVelocityCoefficient * DeltaSeconds);
				const FVector NewAngVel = FVector(NewState.AngVel) + (AngDiffAxis * AngDiff * AngularVelocityCoefficient * DeltaSeconds);

				const FVector NewPos = FMath::Lerp(FVector(CurrentState.Position), FVector(TargetPos), PositionLerp);
				const FQuat NewAng = FQuat::Slerp(CurrentState.Quaternion, TargetQuat, AngleLerp);

				BI->SetBodyTransform(FTransform(NewAng, NewPos), ETeleportType::ResetPhysics);
				BI->SetLinearVelocity(NewLinVel, false);
				BI->SetAngularVelocityInRadians(FMath::DegreesToRadians(NewAngVel), false);
			}
			else
			{
				//If async is used, enqueue for callback
				FPhysicsRepAsyncInputData AsyncInputData(nullptr);
				AsyncInputData.TargetState = NewState;
				AsyncInputData.TargetState.Position = IdealWorldTM.GetLocation();
				AsyncInputData.TargetState.Quaternion = IdealWorldTM.GetRotation();
				AsyncInputData.Proxy = static_cast<Chaos::FSingleParticlePhysicsProxy*>(BI->GetPhysicsActor());
				AsyncInputData.ErrorCorrection = { ErrorCorrection.LinearVelocityCoefficient, ErrorCorrection.AngularVelocityCoefficient, ErrorCorrection.PositionLerp, ErrorCorrection.AngleLerp };
				AsyncInputData.LatencyOneWay = PingSeconds;

				AsyncInput->InputData.Add(AsyncInputData);
			}
		}

		// Should we show the async part?
#if !UE_BUILD_SHIPPING
		if (CharacterMovementCVars::NetShowCorrections != 0)
		{
			PhysicsTarget.ErrorHistory.bAutoAdjustMinMax = false;
			PhysicsTarget.ErrorHistory.MinValue = 0.0f;
			PhysicsTarget.ErrorHistory.MaxValue = 1.0f;
			PhysicsTarget.ErrorHistory.AddSample(PhysicsTarget.AccumulatedErrorSeconds / ErrorAccumulationSeconds);
			if (UWorld* OwningWorld = GetOwningWorld())
			{
				FColor Color = FColor::White;
				DrawDebugDirectionalArrow(OwningWorld, CurrentState.Position, TargetPos, 5.0f, Color, false, CharacterMovementCVars::NetCorrectionLifetime, 0, 1.5f);
				DrawDebugFloatHistory(*OwningWorld, PhysicsTarget.ErrorHistory, CurrentState.Position + FVector(0.0f, 0.0f, 100.0f), FVector2D(100.0f, 50.0f), FColor::White, false, 0, -1);
			}
		}
#endif
	}

	/////// SLEEP UPDATE ///////
	if (bShouldSleep)
	{
		// In the async case, we apply sleep state in ApplyAsyncDesiredState
		if (PhysicsReplicationAsync == nullptr)
		{
			BI->PutInstanceToSleep();
		}
	}

	PhysicsTarget.PrevPosTarget = TargetPos;
	PhysicsTarget.PrevPos = FVector(CurrentState.Position);

	return bRestoredState;
}


void FPhysicsReplication::PrepareAsyncData_External(const FRigidBodyErrorCorrection& ErrorCorrection)
{
	//todo move this logic into a common function?
	const float PositionLerp = CharacterMovementCVars::PositionLerp >= 0.0f ? CharacterMovementCVars::PositionLerp : ErrorCorrection.PositionLerp;
	const float LinearVelocityCoefficient = CharacterMovementCVars::LinearVelocityCoefficient >= 0.0f ? CharacterMovementCVars::LinearVelocityCoefficient : ErrorCorrection.LinearVelocityCoefficient;
	const float AngleLerp = CharacterMovementCVars::AngleLerp >= 0.0f ? CharacterMovementCVars::AngleLerp : ErrorCorrection.AngleLerp;
	const float AngularVelocityCoefficient = CharacterMovementCVars::AngularVelocityCoefficient >= 0.0f ? CharacterMovementCVars::AngularVelocityCoefficient : ErrorCorrection.AngularVelocityCoefficient;

	AsyncInput = PhysicsReplicationAsync->GetProducerInputData_External();
	AsyncInput->ErrorCorrection.PositionLerp = PositionLerp;
	AsyncInput->ErrorCorrection.AngleLerp = AngleLerp;
	AsyncInput->ErrorCorrection.LinearVelocityCoefficient = LinearVelocityCoefficient;
	AsyncInput->ErrorCorrection.AngularVelocityCoefficient = AngularVelocityCoefficient;
}

#pragma region FPhysicsReplicationAsync
void FPhysicsReplicationAsync::OnPhysicsObjectUnregistered_Internal(Chaos::FConstPhysicsObjectHandle PhysicsObject)
{
	RemoveObjectFromReplication(PhysicsObject);

	// Only clear Settings when PhysicsObject unregister (not when it stops replicating, hence why it's not baked into RemoveObjectFromReplication())
	ObjectToSettings.Remove(PhysicsObject);
	RemoveParticleSimDecaySettings(PhysicsObject);

	Chaos::FReadPhysicsObjectInterface_Internal Interface = Chaos::FPhysicsObjectInternalInterface::GetRead();
	if (Chaos::FGeometryParticleHandle* Handle = Interface.GetParticle(PhysicsObject))
	{
		if (Chaos::FPhysicsSolverBase* SolverBase = GetSolver())
		{
			if (Chaos::FRewindData* RewindData = SolverBase->GetRewindData())
			{
				// Stop forcing resim as follower 
				RewindData->ForceResimAsFollower(Handle, false);
			}
		}
	}
}

void FPhysicsReplicationAsync::RegisterSettings(Chaos::FConstPhysicsObjectHandle PhysicsObject, TWeakPtr<const FNetworkPhysicsSettingsData> InSettings)
{
	if (PhysicsObject != nullptr)
	{
		TWeakPtr<const FNetworkPhysicsSettingsData>& Settings = ObjectToSettings.FindOrAdd(PhysicsObject);
		Settings = InSettings;
	}
}

TWeakPtr<FParticleSimDecaySettings> FPhysicsReplicationAsync::FindOrAddParticleSimDecaySettings(Chaos::FConstPhysicsObjectHandle PhysicsObject)
{
	if (PhysicsObject == nullptr)
	{
		return nullptr;
	}

	// Caller writes directly into the returned element
	TSharedPtr<FParticleSimDecaySettings>& Entry = ParticleSimDecaySettings.FindOrAdd(PhysicsObject);
	if (!Entry.IsValid())
	{
		Entry = MakeShared<FParticleSimDecaySettings>();
	}
	return Entry.ToWeakPtr();
}

void FPhysicsReplicationAsync::RemoveParticleSimDecaySettings(Chaos::FConstPhysicsObjectHandle PhysicsObject)
{
	ParticleSimDecaySettings.Remove(PhysicsObject);
}

void FPhysicsReplicationAsync::FetchObjectSettings(Chaos::FConstPhysicsObjectHandle PhysicsObject)
{
	TWeakPtr<const FNetworkPhysicsSettingsData>* CustomSettings = ObjectToSettings.Find(PhysicsObject);
	SettingsCurrent = (CustomSettings && (*CustomSettings).IsValid()) ? *(*CustomSettings).Pin().Get() : SettingsDefault;
}

void FPhysicsReplicationAsync::OnPostInitialize_Internal()
{
	Chaos::FPBDRigidsSolver* RigidsSolver = static_cast<Chaos::FPBDRigidsSolver*>(GetSolver());
	if (RigidsSolver == nullptr)
	{
		return;
	}

	RigidsSolver->SetPhysicsReplication_Internal(this);
}

void FPhysicsReplicationAsync::AddResimulationRequest_Internal(const float DeltaSeconds)
{
	//NOTE: This gets called if Chaos::RewindBeforeAdvance != 0 before the current physics frame execute any logic or callbacks

	if (FPhysicsReplication::ShouldSkipPhysicsReplication())
	{
		return;
	}

	const FPhysicsReplicationAsyncInput* AsyncInput = GetConsumerInput_Internal();
	if (!AsyncInput)
	{
		return;
	}

	NetworkPhysicsTickOffsetAssigned = AsyncInput->NetworkPhysicsTickOffsetAssigned;
	NetworkPhysicsTickOffset = AsyncInput->NetworkPhysicsTickOffset;

	Chaos::FPBDRigidsSolver* RigidsSolver = static_cast<Chaos::FPBDRigidsSolver*>(GetSolver());
	check(RigidsSolver);
	Chaos::FRewindData* RewindData = RigidsSolver->GetRewindData();

	Chaos::FReadPhysicsObjectInterface_Internal Interface = Chaos::FPhysicsObjectInternalInterface::GetRead();
	const int32 CurrentSolverFrame = RigidsSolver->GetCurrentFrame();

	// Pass 1: per-asyncinput ingestion of received target states.
	for (const FPhysicsRepAsyncInputData& InputData : AsyncInput->InputData)
	{
		if (InputData.TargetState.Flags == ERigidBodyFlags::None)
		{
			RemoveObjectFromReplication(InputData.PhysicsObject);
			continue;
		}

		UpdateRewindDataTarget(InputData);
		UpdateAsyncTarget(InputData, RigidsSolver);
	}

	// Pass 2: per-particle LOD + resim trigger
	for (const FPhysicsRepAsyncInputData& InputData : AsyncInput->InputData)
	{
		if (InputData.TargetState.Flags == ERigidBodyFlags::None)
		{
			continue;
		}

		FReplicatedPhysicsTargetAsync* Target = ObjectToTarget.Find(InputData.PhysicsObject);
		if (!Target)
		{
			continue;
		}

		if (Target->LastLODFrame == CurrentSolverFrame)
		{
			// Already processed this particle this frame, can happen if AsyncInput holds multiple InputData for the same particle
			continue;
		}

		Target->SimDecayTimeScale = 1.0f;
		ApplyPhysicsReplicationLOD(InputData.PhysicsObject, *Target, EPhysicsReplicationLODFlags::LODFlag_All);

		if (Target->RepMode != EPhysicsReplicationMode::Resimulation)
		{
			continue;
		}

		// Enable rewind capture lazily on first sight of a resim-mode particle.
		if (!RewindData)
		{
			RewindData = GetOrEnableRewindData(RigidsSolver);
		}

		if (!RewindData || !IsTargetValidForResim(*Target))
		{
			continue;
		}

		Chaos::FPBDRigidParticleHandle* RigidHandle = Interface.GetRigidParticle(InputData.PhysicsObject);
		if (!RigidHandle)
		{
			continue;
		}

		// Target->ServerFrame is the latest after pass 1.
		const int32 LocalFrame = Target->ServerFrame - Target->FrameOffset;
		if (ShouldResimulateParticle(RigidHandle, *Target, DeltaSeconds) && LocalFrame > RewindData->GetBlockedResimFrame())
		{
			RewindData->RequestResimulation(LocalFrame, RigidHandle);
		}
	}
}

void FPhysicsReplicationAsync::OnPreSimulate_Internal()
{
	const FPhysicsReplicationAsyncInput* AsyncInput = GetConsumerInput_Internal();

	if (AsyncInput)
	{
		NetworkPhysicsTickOffsetAssigned = AsyncInput->NetworkPhysicsTickOffsetAssigned;
		NetworkPhysicsTickOffset = AsyncInput->NetworkPhysicsTickOffset;
	}

	if (FPhysicsReplication::ShouldSkipPhysicsReplication())
	{
		return;
	}

	Chaos::FPBDRigidsSolver* RigidsSolver = static_cast<Chaos::FPBDRigidsSolver*>(GetSolver());
	check(RigidsSolver);

	ResimErrorLogTimer += RigidsSolver->GetAsyncDeltaTime();

	// Early out if this is a resim frame
	Chaos::FRewindData* RewindData = RigidsSolver->GetRewindData();
	bool bRewindDataExist = RewindData != nullptr;
	if (bRewindDataExist && RewindData->IsResim())
	{
		// TODO, Handle the transition from post-resim to interpolation better (disabled by default, resim vs replication interaction is handled via FPhysicsReplicationAsync::CacheResimInteractions)
		if (SettingsCurrent.PredictiveInterpolationSettings.GetPostResimWaitForUpdate() && RewindData->IsFinalResim())
		{
			for (auto Itr = ObjectToTarget.CreateIterator(); Itr; ++Itr)
			{
				FReplicatedPhysicsTargetAsync& Target = Itr.Value();

				// If final resim frame, mark interpolated targets as waiting for up to date data from the server.
				if (Target.RepMode == EPhysicsReplicationMode::PredictiveInterpolation)
				{
					Target.SetWaiting(RigidsSolver->GetCurrentFrame() + Target.FrameOffset, Target.RepModeOverride);
				}
			}
		}
		return;
	}

	if (AsyncInput)
	{
		for (const FPhysicsRepAsyncInputData& Input : AsyncInput->InputData)
		{
			if (Input.TargetState.Flags == ERigidBodyFlags::None)
			{
				RemoveObjectFromReplication(Input.PhysicsObject);
				continue;
			}

			// Verify the received target is on the same quantization grid as used for per-frame physics quantization
			if (Input.ServerFrame > 0)
			{
				UE::PhysicsReplicationQuantization::VerifyTargetOnGrid_Internal(
					GetSolver(), Input.PhysicsObject, Input.ServerFrame,
					Input.TargetState.Position, Input.TargetState.Quaternion,
					Input.TargetState.LinVel, Input.TargetState.AngVel);
			}

			// When Chaos::RewindBeforeAdvance != 0 these writes are done inside AddResimulationRequest_Internal()'s Pass 1 instead. Running them again here would clobber Target state established by Pass 2's LOD (TickCount, SimDecayTimeScale, the cached helpers). RewindCapture is lazy-enabled at the RequestResimulation call sites, so UpdateRewindDataTarget may no-op until the first resim trigger forces it on.
			if (Chaos::RewindBeforeAdvance == 0)
			{
				UpdateRewindDataTarget(Input);
				UpdateAsyncTarget(Input, RigidsSolver);
			}

			// Deprecated, legacy BodyInstance flow for Default Replication
			if (Input.Proxy != nullptr)
			{
				Chaos::FSingleParticlePhysicsProxy* Proxy = Input.Proxy;
				Chaos::FRigidBodyHandle_Internal* Handle = Proxy->GetPhysicsThreadAPI();

				const FPhysicsRepErrorCorrectionData& UsedErrorCorrection = Input.ErrorCorrection.IsSet() ? Input.ErrorCorrection.GetValue() : AsyncInput->ErrorCorrection;
				DefaultReplication_DEPRECATED(Handle, Input, GetDeltaTime_Internal(), UsedErrorCorrection);

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
				if (PhysicsReplicationCVars::DebugDrawShowRepMode)
				{
					Chaos::FDebugDrawQueue::GetInstance().DrawDebugBox(Input.TargetState.Position, FVector(10.0f, 10.0f, 10.0f), Input.TargetState.Quaternion, FColor::Green, false, PhysicsReplicationCVars::DebugDrawLifeTime, 0, 1.0f);
				}
#endif
			}
		}
	}

	if (Chaos::FPBDRigidsSolver::IsNetworkPhysicsPredictionEnabled())
	{
		CacheResimInteractions();
	}

	ApplyTargetStatesAsync(GetDeltaTime_Internal());
}

void FPhysicsReplicationAsync::OnPostIntegrate_Internal()
{
	Chaos::FPBDRigidsSolver* RigidsSolver = static_cast<Chaos::FPBDRigidsSolver*>(GetSolver());
	if (!RigidsSolver)
	{
		return;
	}

	Chaos::FRewindData* RewindData = RigidsSolver->GetRewindData();
	const bool bIsResim = RewindData && RewindData->IsResim();

	Chaos::FWritePhysicsObjectInterface_Internal Interface = Chaos::FPhysicsObjectInternalInterface::GetWrite();

	// Scale back the integration delta between Pre- and Post-integrate by TimeScale
	auto ApplyDecay = [&](Chaos::FConstPhysicsObjectHandle Handle, float TimeScale)
	{
		if (TimeScale >= 1.0f)
		{
			return;
		}

		Chaos::FPBDRigidParticleHandle* Particle = Interface.GetRigidParticle(Handle);
		if (!Particle || !Particle->IsDynamic())
		{
			return;
		}

		const Chaos::FVec3 PrePos = Particle->GetTransformXRCom().GetLocation();
		const Chaos::FRotation3 PreRot = Particle->GetTransformXRCom().GetRotation();
		const Chaos::FVec3 PreVel = Particle->GetPreV();
		const Chaos::FVec3 PreAngVel = Particle->GetPreW();

		const Chaos::FVec3 PostPos = Particle->GetTransformPQCom().GetLocation();
		const Chaos::FRotation3 PostRot = Particle->GetTransformPQCom().GetRotation();
		const Chaos::FVec3 PostVel = Particle->GetV();
		const Chaos::FVec3 PostAngVel = Particle->GetW();

		Particle->SetTransformPQCom(
			PrePos + TimeScale * (PostPos - PrePos),
			Chaos::FRotation3::Slerp(PreRot, PostRot, TimeScale).GetNormalized());
		Particle->SetV(PreVel + TimeScale * (PostVel - PreVel));
		Particle->SetW(PreAngVel + TimeScale * (PostAngVel - PreAngVel));

		if (PhysicsReplicationLODCVars::bTransitionModeDebugLog)
		{
			const double DeltaLen = (PostPos - PrePos).Size();
			const double ScaledLen = DeltaLen * TimeScale;
			UE_LOGF(LogPhysics, Log, "[ReplicationLOD] Decay | Frame=%d Resim=%d TimeScale=%.3f | DeltaLen=%.3f ScaledLen=%.3f | PrePos=%.2f PostPos=%.2f ResultPos=%.2f"
				, RigidsSolver->GetCurrentFrame(), bIsResim ? 1 : 0, TimeScale, DeltaLen, ScaledLen, PrePos.Size(), PostPos.Size(), (PrePos + TimeScale * (PostPos - PrePos)).Size());
		}
	};


	if (bIsResim)
	{
		// On first resim step, min-merge per-particle FParticleSimDecaySettings into Target.SimDecayTimeScale
		if (RigidsSolver->GetCurrentFrame() == RewindData->GetResimFrame())
		{
			const float ResimFrames = static_cast<float>(RewindData->GetLatestFrame() - RewindData->GetResimFrame());

			for (const TPair<Chaos::FConstPhysicsObjectHandle, TSharedPtr<FParticleSimDecaySettings>>& Pair : ParticleSimDecaySettings)
			{
				FReplicatedPhysicsTargetAsync* ClampTarget = ObjectToTarget.Find(Pair.Key);
				if (!ClampTarget)
				{
					continue;
				}

				const TSharedPtr<FParticleSimDecaySettings>& Settings = Pair.Value;

				float TimeScale = Settings->StaticTimeScale;
				if (Settings->bUseDynamicTimeScale && ResimFrames > 0.0f)
				{
					TimeScale = FMath::Clamp(
						Settings->DynamicBase + (ResimFrames - Settings->InputPredictionFramesAverage) / ResimFrames,
						Settings->DynamicMin, Settings->DynamicMax);
				}

				ClampTarget->SimDecayTimeScale = FMath::Min(ClampTarget->SimDecayTimeScale, TimeScale);

				if (PhysicsReplicationLODCVars::bTransitionModeDebugLog)
				{
					UE_LOGF(LogPhysics, Log, "[ReplicationLOD] Clamp | ActualResimFrames=%.0f InputPredAvg=%.2f UseDynamic=%d | TimeScale=%.3f SimDecayTimeScale=%.3f (post min-merge)"
						, ResimFrames, Settings->InputPredictionFramesAverage, Settings->bUseDynamicTimeScale ? 1 : 0
						, TimeScale, ClampTarget->SimDecayTimeScale);
				}
			}
		}

		// Per-step decay during resim, applied to every replicated Target whose SimDecayTimeScale was reduced
		for (const TPair<Chaos::FConstPhysicsObjectHandle, FReplicatedPhysicsTargetAsync>& Pair : ObjectToTarget)
		{
			ApplyDecay(Pair.Key, Pair.Value.SimDecayTimeScale);
		}
	}
	else
	{
		// Runtime decay path: opt-in via Settings.bApplyDecayAtRuntime, suppressed when LOD has forced PI on the particle
		for (const TPair<Chaos::FConstPhysicsObjectHandle, TSharedPtr<FParticleSimDecaySettings>>& Pair : ParticleSimDecaySettings)
		{
			if (!Pair.Value->bApplyDecayAtRuntime)
			{
				continue;
			}

			// Apply if we either don't have a target or if the target uses EPhysicsReplicationMode::Resimulation
			// TODO: Use persistent particle replication mode API when implemented
			const FReplicatedPhysicsTargetAsync* Target = ObjectToTarget.Find(Pair.Key);
			if (Target && Target->RepMode != EPhysicsReplicationMode::Resimulation)
			{
				continue;
			}

			ApplyDecay(Pair.Key, Pair.Value->StaticTimeScale);
		}
	}
}

void FPhysicsReplicationAsync::OnPostSolve_Internal()
{
	if (FPhysicsReplication::ShouldSkipPhysicsReplication() || PhysicsReplicationCVars::ResimulationCVars::bKeepResimStateForNonResimReplicatedObjects)
	{
		return;
	}

	Chaos::FPBDRigidsSolver* RigidsSolver = static_cast<Chaos::FPBDRigidsSolver*>(GetSolver());
	check(RigidsSolver);

	// On final resimulation frame, revert non-resim replicated objects to their pre-resim state
	Chaos::FRewindData* RewindData = RigidsSolver->GetRewindData();
	if (RewindData && RewindData->IsFinalResim())
	{
		Chaos::FReadPhysicsObjectInterface_Internal Interface = Chaos::FPhysicsObjectInternalInterface::GetRead();
		for (auto Itr = ObjectToTarget.CreateIterator(); Itr; ++Itr)
		{
			FReplicatedPhysicsTargetAsync& Target = Itr.Value();
			if (Target.RepMode == EPhysicsReplicationMode::Default || Target.RepMode == EPhysicsReplicationMode::PredictiveInterpolation)
			{
				Chaos::FConstPhysicsObjectHandle& PhysicsObject = Itr.Key();
				if (Chaos::FGeometryParticleHandle* GeometryParticle = Interface.GetParticle(PhysicsObject))
				{
					Chaos::FParticleSimpleState* PreResimState = nullptr;
					RewindData->GetPreResimState(GeometryParticle, PreResimState);
					if (PreResimState)
					{
						if (Chaos::FPBDRigidParticleHandle* RigidParticle = GeometryParticle->CastToRigidParticle())
						{
							// Note, PostSolve happens before rigids update XR from PQ
							RigidParticle->SetP(PreResimState->GetX());
							RigidParticle->SetQ(PreResimState->GetR());
							RigidParticle->SetV(PreResimState->GetV());
							RigidParticle->SetW(PreResimState->GetW());
						}
						else if (Chaos::FKinematicGeometryParticleHandle* KinematicParticle = GeometryParticle->CastToKinematicParticle())
						{
							// Kinematic particles doesn't have PR so set XR directly
							KinematicParticle->SetX(PreResimState->GetX());
							KinematicParticle->SetR(PreResimState->GetR());
							KinematicParticle->SetV(PreResimState->GetV());
							KinematicParticle->SetW(PreResimState->GetW());
						}

						// Remove pre resim state
						RewindData->RemovePreResimState(GeometryParticle);
					}
				}
			}
		}
	}
}

FReplicatedPhysicsTargetAsync* FPhysicsReplicationAsync::AddObjectToReplication(Chaos::FConstPhysicsObjectHandle PhysicsObject)
{
	if (ensure(PhysicsObject))
	{
		// Cache ParticleID in array of replicated objects
		Chaos::FReadPhysicsObjectInterface_Internal Interface = Chaos::FPhysicsObjectInternalInterface::GetRead();
		if (Chaos::FGeometryParticleHandle* Handle = Interface.GetParticle(PhysicsObject))
		{
			ReplicatedParticleIDs.Add(Handle->ParticleID());
		}

		// Add to Object-Target map
		return &ObjectToTarget.Add(PhysicsObject, FReplicatedPhysicsTargetAsync());
	}
	return nullptr;
}

void FPhysicsReplicationAsync::RemoveObjectFromReplication(Chaos::FConstPhysicsObjectHandle PhysicsObject)
{
	if (PhysicsObject == nullptr)
	{
		return;
	}

	// Remove from Object-Target map
	ObjectToTarget.Remove(PhysicsObject);

	// Remove cached replicated ParticleID
	Chaos::FReadPhysicsObjectInterface_Internal Interface = Chaos::FPhysicsObjectInternalInterface::GetRead();
	if (Chaos::FGeometryParticleHandle* Handle = Interface.GetParticle(PhysicsObject))
	{
		ReplicatedParticleIDs.Remove(Handle->ParticleID());
	}
}

void FPhysicsReplicationAsync::UpdateRewindDataTarget(const FPhysicsRepAsyncInputData& Input)
{
	if (Input.PhysicsObject == nullptr)
	{
		return;
	}

	// If there is no FrameOffset set then we have not synced up physics ticks with the server yet so don't cache this data
	if (NetworkPhysicsTickOffsetAssigned == false)
	{
		return;
	}

	Chaos::FPBDRigidsSolver* RigidsSolver = static_cast<Chaos::FPBDRigidsSolver*>(GetSolver());
	if (RigidsSolver == nullptr)
	{
		return;
	}

	Chaos::FRewindData* RewindData = RigidsSolver->GetRewindData();
	if (RewindData == nullptr)
	{
		return;
	}

	Chaos::FReadPhysicsObjectInterface_Internal Interface = Chaos::FPhysicsObjectInternalInterface::GetRead();
	if (Chaos::FGeometryParticleHandle* Handle = Interface.GetParticle(Input.PhysicsObject))
	{
		// Cache all target states inside RewindData
		const int32 LocalFrame = Input.ServerFrame - NetworkPhysicsTickOffset;
		RewindData->SetTargetStateAtFrame(*Handle, LocalFrame, Chaos::FFrameAndPhase::EParticleHistoryPhase::PrePushData,
			Input.TargetState.Position, Input.TargetState.Quaternion,
			Input.TargetState.LinVel, FMath::DegreesToRadians(Input.TargetState.AngVel), (Input.TargetState.Flags & ERigidBodyFlags::Sleeping));

		RewindData->ForceResimAsFollower(Handle, /*bForceResim*/ Input.RepMode == EPhysicsReplicationMode::None);
	}
}

void FPhysicsReplicationAsync::UpdateAsyncTarget(const FPhysicsRepAsyncInputData& Input, Chaos::FPBDRigidsSolver* RigidsSolver)
{
	if (Input.PhysicsObject == nullptr)
	{
		return;
	}

	FReplicatedPhysicsTargetAsync* Target = ObjectToTarget.Find(Input.PhysicsObject);
	bool bFirstTarget = Target == nullptr;
	if (bFirstTarget)
	{
		// First time we add a target, set previous state to current input
		Target = AddObjectToReplication(Input.PhysicsObject);
		Target->PrevPos = Input.TargetState.Position;
		Target->PrevPosTarget = Input.TargetState.Position;
		Target->PrevRotTarget = Input.TargetState.Quaternion;
		Target->PrevLinVel = Input.TargetState.LinVel;
		Target->RepModeOverride = Input.RepMode;
	}
	check(Target);

	/** Target Update Description
	* @param Input = incoming state target for replication.
	* 
	* Input comes mainly from the server but can be a faked state produced by the client for example if the client object wakes up from sleeping.
	* Fake inputs should have a ServerFrame of -1 (bool bIsFake = Input.ServerFrame == -1)
	* Server inputs can have ServerFrame values of either 0 or an incrementing integer value.
	*	If the ServerFrame is 0 it should always be 0. If it's incrementing it will always increment.
	*
	* @local Target = The current state target used for replication, to be updated with data from Input.
	* Read about the different target properties in FReplicatedPhysicsTargetAsync
	* 
	* IMPORTANT:
	* Target.ServerFrame can be -1 if the target is newly created or if it has data from a fake input.
	* 
	* SendInterval is calculated by taking Input.ServerFrame - Target.ServerFrame
	*	Note, can only be calculated if the server is sending incrementing SendIntervals and if we have received a valid input previously so we have the previous ServerFrame cached in Target.
	* 
	* ReceiveInterval is calculated by taking RigidsSolver->GetCurrentFrame() - Target.ReceiveFrame
	*	Note that ReceiveInterval is only used if SendInterval is 0
	* 
	* Target.TickCount starts at 0 and is incremented each tick that the target is used for, TickCount is reset back to 0 each time Target is updated with new Input.
	* 
	* NOTE: With perfect network conditions SendInterval, ReceiveInterval and Target.TickCount will be the same value.
	*/

	// Update target from input if input is newer than target or this is the first input received (target is empty)
	if ((bFirstTarget || Input.ServerFrame == 0 || Input.ServerFrame > Target->ServerFrame))
	{
		// Get the current physics frame
		const int32 CurrentFrame = RigidsSolver->GetCurrentFrame();

		// Cache TickCount before updating it, force to 0 if ServerFrame is -1
		const int32 PrevTickCount = (Target->ServerFrame < 0) ? 0 : Target->TickCount;

		// Cache SendInterval, only calculate if we have a valid Target->ServerFrame, else leave at 0.
		const int32 SendInterval = (Target->ServerFrame <= 0) ? 0 : Input.ServerFrame - Target->ServerFrame;

		// Cache if this target was previously allowed to be altered, before this update
		const bool bPrevAllowTargetAltering = Target->bAllowTargetAltering;

		// Cache if the physics frame offset has changed since last target
		const bool bFrameOffsetCorrected = Target->FrameOffset != NetworkPhysicsTickOffset;

		// Set if the target is allowed to be altered after this update
		Target->bAllowTargetAltering = !(Target->TargetState.Flags & ERigidBodyFlags::Sleeping) && !(Input.TargetState.Flags & ERigidBodyFlags::Sleeping);

		// Cache previous linear velocity
		const FVector PrevLinVel = Target->TargetState.LinVel;

		// Set Target->ReceiveInterval from either SendInterval or the number of physics ticks between receiving input states
		if (SendInterval > 0)
		{
			Target->ReceiveInterval = SendInterval;
		}
		else
		{
			const int32 PrevReceiveFrame = Target->ReceiveFrame < 0 ? (CurrentFrame - 1) : Target->ReceiveFrame;
			Target->ReceiveInterval = (CurrentFrame - PrevReceiveFrame);
		}

		// Update target from input and reset properties
		Target->ServerFrame = Input.ServerFrame;
		Target->ReceiveFrame = CurrentFrame;
		Target->TargetState = Input.TargetState;
		Target->RepMode = Input.RepMode;
		Target->FrameOffset = NetworkPhysicsTickOffset;
		Target->TickCount = 0;
		Target->AccumulatedSleepSeconds = 0.0f;
		Target->SimDecayTimeScale = 1.0f;

		// Update waiting state
		Target->UpdateWaiting(Input.ServerFrame);

		// Per-frame transient helper properties to continue Target update later in the frame
		Target->bFrameOffsetCorrectedCached = bFrameOffsetCorrected;
		Target->PrevTickCountCached = PrevTickCount;
		Target->bPrevAllowTargetAlteringCached = bPrevAllowTargetAltering;

		// Teleport detection, we don't have specific data that tells us a teleport has happened on the server, so try to detect it by examining the previous and next state
		if (PhysicsReplicationCVars::PredictiveInterpolationCVars::TeleportDetectionEnabled == 1 && !bFirstTarget && SendInterval > 0 && RigidsSolver->IsUsingFixedDt())
		{
			const FVector PosOffset = (Input.TargetState.Position - Target->PrevPosTarget);
			if (PosOffset.SizeSquared() > (PhysicsReplicationCVars::PredictiveInterpolationCVars::TeleportDetectionMinDistance * PhysicsReplicationCVars::PredictiveInterpolationCVars::TeleportDetectionMinDistance))
			{
				const FVector Velocity = Input.TargetState.LinVel.SizeSquared() > PrevLinVel.SizeSquared() ? Input.TargetState.LinVel : PrevLinVel;
				const float DeltaSeconds = (SendInterval * RigidsSolver->GetAsyncDeltaTime());
				const float PossibleDistanceSquared = (Velocity * (DeltaSeconds * PhysicsReplicationCVars::PredictiveInterpolationCVars::TeleportDetectionVelocityMultiplier)).SizeSquared();

				if (PossibleDistanceSquared < PosOffset.SizeSquared())
				{
					// A teleport has most likely happened, set accumulated error seconds to above limit for hard snapping
					// TODO: Don't piggyback on AccumulatedErrorSeconds (potentially implement ERigidBodyFlags::Teleported)
					Target->AccumulatedErrorSeconds = PhysicsReplicationCVars::PredictiveInterpolationCVars::ErrorAccumulationSeconds + 1.0f;
				}
			}
		}

		// Cache the position we received this target at, Predictive Interpolation will alter the target state but use this as the source position for reconciliation.
		Target->PrevPosTarget = Input.TargetState.Position;
		Target->PrevRotTarget = Input.TargetState.Quaternion;
	}

	/** Cache the latest ping time */
	LatencyOneWay = Input.LatencyOneWay;
}

void FPhysicsReplicationAsync::CacheResimInteractions()
{
	if(!PhysicsReplicationCVars::ResimulationCVars::bDisableReplicationOnInteraction)
	{
		ParticlesInResimIslands.Empty();
		return;
	}

	if (UsePhysicsReplicationLOD())
	{
		// This will be handled by the LOD system
		ParticlesInResimIslands.Empty();
		return;
	}

	Chaos::FPBDRigidsSolver* RigidsSolver = static_cast<Chaos::FPBDRigidsSolver*>(GetSolver());
	if (RigidsSolver == nullptr)
	{
		return;
	}

	ResimIslands.Reset();
	ResimIslandsParticles.Reset();
	ParticlesInResimIslands.Reset();

	Chaos::Private::FPBDIslandManager& IslandManager = RigidsSolver->GetEvolution()->GetIslandManager();
	Chaos::FReadPhysicsObjectInterface_Internal Interface = Chaos::FPhysicsObjectInternalInterface::GetRead();

	for (auto Itr = ObjectToTarget.CreateIterator(); Itr; ++Itr)
	{
		FReplicatedPhysicsTargetAsync& Target = Itr.Value();
		if (Target.RepMode == EPhysicsReplicationMode::Resimulation)
		{
			Chaos::FConstPhysicsObjectHandle& POHandle = Itr.Key();
			if (Chaos::FGeometryParticleHandle* Handle = Interface.GetParticle(POHandle))
			{
				// Get a list of particles from the same island as a resim particle is in, i.e. particles interacting with a resim particle
				IslandManager.FindParticleIslands(Handle, OUT ResimIslands);
				IslandManager.FindParticlesInIslands(ResimIslands, OUT ResimIslandsParticles);
				for (const Chaos::FGeometryParticleHandle* InteractParticle : ResimIslandsParticles)
				{
					ParticlesInResimIslands.Add(InteractParticle->GetHandleIdx());
				}
			}
		}
	}
}

void FPhysicsReplicationAsync::ApplyTargetStatesAsync(const float DeltaSeconds)
{
	using namespace Chaos;
	Chaos::FPBDRigidsSolver* RigidsSolver = static_cast<Chaos::FPBDRigidsSolver*>(GetSolver());
	if (!RigidsSolver)
	{
		return;
	}
	const int32 CurrentSolverFrame = RigidsSolver->GetCurrentFrame();

	/** Helper function to remove replicated target*/
	auto RemoveTargetHelper = [this](TMap<Chaos::FConstPhysicsObjectHandle, FReplicatedPhysicsTargetAsync>::TIterator Itr, FGeometryParticleHandle* Handle)
	{
		if (Handle)
		{
			ReplicatedParticleIDs.Remove(Handle->ParticleID());
		}
		PendingDeleteFromObjectToTarget.Add(Itr.Key());
	};

	// PhysicsObject flow
	Chaos::FWritePhysicsObjectInterface_Internal Interface = Chaos::FPhysicsObjectInternalInterface::GetWrite();
	for (TMap<Chaos::FConstPhysicsObjectHandle, FReplicatedPhysicsTargetAsync>::TIterator Itr = ObjectToTarget.CreateIterator(); Itr; ++Itr)
	{
		bool bRemoveItr = true; // Remove current cached replication target unless replication logic tells us to store it for next tick

		Chaos::FConstPhysicsObjectHandle& POHandle = Itr.Key();
		FGeometryParticleHandle* Handle = Interface.GetParticle(POHandle);
		if (!Handle)
		{
			RemoveTargetHelper(Itr, nullptr);
			continue;
		}

		FPBDRigidParticleHandle* RigidHandle = Handle->CastToRigidParticle();
		if (!RigidHandle)
		{
			RemoveTargetHelper(Itr, Handle);
			continue;
		}

		FReplicatedPhysicsTargetAsync& Target = Itr.Value();

		// Cache custom settings for this object if there are any
		FetchObjectSettings(POHandle);

		const bool bFreshInputThisFrame = (Target.ReceiveFrame == CurrentSolverFrame);

		// Run LOD if not already done this frame
		if (Target.LastLODFrame != CurrentSolverFrame)
		{
			Target.SimDecayTimeScale = 1.0f;
			ApplyPhysicsReplicationLOD(POHandle, Target, EPhysicsReplicationLODFlags::LODFlag_All);
		}

		// Check new targets for resim validity
		if (bFreshInputThisFrame)
		{
			CheckTargetResimValidity(Target);
		}

		// Predictive Interpolation prep
		if (Target.RepMode == EPhysicsReplicationMode::PredictiveInterpolation)
		{
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
			if (PhysicsReplicationCVars::PredictiveInterpolationCVars::bDrawDebugTargets)
			{
				const FVector Offset = FVector(0.0f, 0.0f, PhysicsReplicationCVars::PredictiveInterpolationCVars::DrawDebugZOffset);
				Chaos::FDebugDrawQueue::GetInstance().DrawDebugBox(
					Target.PrevPosTarget + Offset, FVector(15.0f, 15.0f, 15.0f),
					Target.PrevRotTarget,
					FColor::MakeRandomSeededColor(Target.ServerFrame),
					false, PhysicsReplicationCVars::DebugDrawLifeTime, 0, 1.0f);
			}
#endif

			// Align target state if new target, TickCount will be 0 if no LOD alignment was performed which means it's okay to perform the normal target alignment.
			if (bFreshInputThisFrame && Target.TickCount == 0 && !Target.bFrameOffsetCorrectedCached)
			{
				if (Target.bPrevAllowTargetAlteringCached && Target.bAllowTargetAltering && !Target.IsWaiting())
				{
					Target.TickCount = Target.PrevTickCountCached - Target.ReceiveInterval;
					const int32 AdjustedAverageReceiveInterval = FMath::CeilToInt(Target.AverageReceiveInterval) * PhysicsReplicationCVars::PredictiveInterpolationCVars::TargetTickAlignmentClampMultiplier;
					const int32 TickAlignment = FMath::Clamp(Target.TickCount, -AdjustedAverageReceiveInterval, AdjustedAverageReceiveInterval);
					FPhysicsReplicationAsync::ExtrapolateTarget(Target, TickAlignment, GetDeltaTime_Internal());
				}
			}
		}

		DebugDrawReplicationMode(POHandle, Target);

		const EPhysicsReplicationMode RepMode = Target.IsWaiting() ? Target.RepModeOverride : Target.RepMode;
		switch (RepMode)
		{
			case EPhysicsReplicationMode::Default:
				bRemoveItr = DefaultReplication(RigidHandle, Target, DeltaSeconds);
				break;

			case EPhysicsReplicationMode::PredictiveInterpolation:
				bRemoveItr = PredictiveInterpolation(RigidHandle, Target, DeltaSeconds);
				break;

			case EPhysicsReplicationMode::Resimulation:
				bRemoveItr = ResimulationReplication(RigidHandle, Target, DeltaSeconds);
				break;

			case EPhysicsReplicationMode::None:
				bRemoveItr = true;
				break;
		}
		Target.TickCount++;

		if (bRemoveItr)
		{
			RemoveTargetHelper(Itr, RigidHandle);
		}
	}

	for (Chaos::FConstPhysicsObjectHandle& PhysicsObject : PendingDeleteFromObjectToTarget)
	{
		ObjectToTarget.Remove(PhysicsObject);
	}
	PendingDeleteFromObjectToTarget.Reset();
}

void FPhysicsReplicationAsync::CheckTargetResimValidity(FReplicatedPhysicsTargetAsync& Target)
{
	if (Target.RepMode != EPhysicsReplicationMode::Resimulation)
	{
		return;
	}

	Chaos::FPBDRigidsSolver* RigidsSolver = static_cast<Chaos::FPBDRigidsSolver*>(GetSolver());
	if (RigidsSolver == nullptr)
	{
		return;
	}

	Chaos::FRewindData* RewindData = GetOrEnableRewindData(RigidsSolver);
	if (RewindData == nullptr)
	{
		// Can't perform resimulation without RewindData, revert to PredictiveInterpolation
		Target.RepMode = EPhysicsReplicationMode::PredictiveInterpolation;
		return;
	}

	const int32 LocalFrame = Target.ServerFrame - NetworkPhysicsTickOffset;
	if (!RewindData->IsFrameWithinRewindHistory(LocalFrame))
	{
		if (LocalFrame < RewindData->GetEarliestFrame_Internal())
		{
			// Client is far ahead of the server, switch over to Predictive Interpolation since it can't use incoming target states from the server to perform resimulations with

			Target.RepMode = EPhysicsReplicationMode::PredictiveInterpolation;
		}
		else if (PhysicsReplicationCVars::ResimulationCVars::bApplyPredictiveInterpolationWhenBehindServer)
		{
			/** NOTE: If the server is ahead of the client we receive target states for frames we have not yet simulated on the client, target states are stored in FRewindData still though.
			* If PhysicsReplicationCVars::ResimulationCVars::bApplyPredictiveInterpolationWhenBehindServer is true switch over to using PredictiveInterpolation temporarily.
			* else FRewindData::CompareTargetsToLastFrame will check for already cached targets to resim with when the server has simulated the corresponding frame */

			Target.RepMode = EPhysicsReplicationMode::PredictiveInterpolation;
		}
		
		if (NetworkPhysicsTickOffsetAssigned)
		{
			if (ResimOutOfBoundsCounter == 0)
			{
				UE_LOGF(LogPhysics, Log, "FPhysicsReplication DESYNCED - received target ClientFrame: %d (ServerFrame: %d - FrameOffset: %d) out of rewind data bounds (%d, %d) - %ls - Target will use %ls"
					, LocalFrame, Target.ServerFrame, NetworkPhysicsTickOffset, RewindData->GetEarliestFrame_Internal(), RewindData->CurrentFrame()
					, (LocalFrame < RewindData->GetEarliestFrame_Internal())
					? TEXT("Client is far ahead of the server, server might be dropping frames.")
					: TEXT("Client is behind the server, client might be dropping frames."), *UEnum::GetValueAsString(Target.RepMode));
			}

			ResimOutOfBoundsCounter++;
			ResimErrorLogTimer = 0;
		}
	}
	else
	{
		if (ResimOutOfBoundsCounter > 0 && ResimErrorLogTimer > PhysicsReplicationCVars::ResimulationCVars::LogOutOfBoundsTimeLimit)
		{
			UE_LOGF(LogPhysics, Log, "FPhysicsReplication IN-SYNC - Received targets have now been within rewind data bounds again for at least %f seconds", ResimErrorLogTimer);

			ResimOutOfBoundsCounter = 0;
		}
	}
}

Chaos::FRewindData* FPhysicsReplicationAsync::GetOrEnableRewindData(Chaos::FPBDRigidsSolver* RigidsSolver)
{
	if (RigidsSolver == nullptr)
	{
		return nullptr;
	}

	Chaos::FRewindData* RewindData = RigidsSolver->GetRewindData();
	if (!RewindData)
	{
		if (Chaos::FPBDRigidsSolver::IsNetworkPhysicsPredictionEnabled() && RigidsSolver->IsUsingFixedDt())
		{
			RigidsSolver->EnableRewindCapture();
			RewindData = RigidsSolver->GetRewindData();
		}
	}
	return RewindData;
}

void FPhysicsReplicationAsync::ApplyPhysicsReplicationLOD(Chaos::FConstPhysicsObjectHandle PhysicsObjectHandle, FReplicatedPhysicsTargetAsync& Target, const uint32 LODFlags)
{
	Chaos::FPBDRigidsSolver* RigidsSolver = static_cast<Chaos::FPBDRigidsSolver*>(GetSolver());
	if (RigidsSolver == nullptr)
	{
		return;
	}
	const int32 CurrentFrame = RigidsSolver->GetCurrentFrame();

	// Temporary target, don't apply LOD until we receive targets for specified server physics frames
	if (Target.ServerFrame <= 0)
	{
		Target.RepMode = EPhysicsReplicationMode::PredictiveInterpolation;
		Target.SimDecayTimeScale = 1.0f;

		if (PhysicsReplicationLODCVars::bTransitionModeDebugLog)
		{
			Chaos::FReadPhysicsObjectInterface_Internal ReadInterface = Chaos::FPhysicsObjectInternalInterface::GetRead();
			const Chaos::FGeometryParticleHandle* DebugHandle = ReadInterface.GetParticle(PhysicsObjectHandle);
			const FVector ParticlePos = DebugHandle ? DebugHandle->GetX() : FVector::ZeroVector;
			const FVector TargetPos = Target.TargetState.Position;
			UE_LOGF(LogPhysics, Log, "[ReplicationLOD] PreServer | CurrentFrame=%d ServerFrame=%d | ParticlePos=(%.2f,%.2f,%.2f) TargetStatePos=(%.2f,%.2f,%.2f) | RepMode=PredictiveInterpolation"
				, RigidsSolver->GetCurrentFrame(), Target.ServerFrame
				, ParticlePos.X, ParticlePos.Y, ParticlePos.Z
				, TargetPos.X, TargetPos.Y, TargetPos.Z);
		}
		return;
	}

	IPhysicsReplicationLODAsync* PhysRepLod = RigidsSolver->GetPhysicsReplicationLOD_Internal();
	if (!PhysRepLod || !PhysRepLod->IsEnabled())
	{
		return;
	}

	FPhysicsRepLodData* LodData = PhysRepLod->GetLODData_Internal(PhysicsObjectHandle, LODFlags);
	if (!LodData || !LodData->DataAssigned)
	{
		return;
	}

	const int32 LocalFrame = Target.ServerFrame - NetworkPhysicsTickOffset;
	const int32 FullPredictionFrames = CurrentFrame - LocalFrame;
	const float DeltaTime = static_cast<float>(RigidsSolver->GetAsyncDeltaTime());
	const float FullPredictionTime = FullPredictionFrames * DeltaTime;

	// Number of frames behind current frame that the LOD wants this particle to replicate at
	const int32 AlignDistance = CurrentFrame - LodData->AlignedFrame;

	/** Outer zone (Predictive Interpolation)
	* - AlignedTime >= FullPredictionTime - LOD wants align the particle further back in time than the full prediction time, meaning the particle is outside of the transition zone.
	* - AlignedFrame == 0 (no focal particle registered) - If we don't get recommended frame to align to we don't have a focal particle in LOD
	* - FullPredictionFrames <= 0 - Server is ahead of client */
	if (LodData->AlignedTime >= FullPredictionTime || LodData->AlignedFrame == 0 || FullPredictionFrames <= 0)
	{
		Target.RepMode = EPhysicsReplicationMode::PredictiveInterpolation;
		Target.SimDecayTimeScale = 1.0f;
		Target.LastLODFrame = CurrentFrame;

		if (PhysicsReplicationLODCVars::bTransitionModeDebugLog)
		{
			Chaos::FReadPhysicsObjectInterface_Internal ReadInterface = Chaos::FPhysicsObjectInternalInterface::GetRead();
			const Chaos::FGeometryParticleHandle* DebugHandle = ReadInterface.GetParticle(PhysicsObjectHandle);
			const FVector ParticlePos = DebugHandle ? DebugHandle->GetX() : FVector::ZeroVector;
			const FVector TargetPos = Target.TargetState.Position;
			UE_LOGF(LogPhysics, Log, "[ReplicationLOD] OuterZone | CurrentFrame=%d LocalFrame=%d AlignedFrame=%d | FullPredictionFrames=%d FullPredictionTime=%.4f AlignedTime=%.4f | No extrapolation | ParticlePos=(%.2f,%.2f,%.2f) TargetStatePos=(%.2f,%.2f,%.2f) | RepMode=PredictiveInterpolation"
				, CurrentFrame, LocalFrame, LodData->AlignedFrame
				, FullPredictionFrames, FullPredictionTime, LodData->AlignedTime
				, ParticlePos.X, ParticlePos.Y, ParticlePos.Z
				, TargetPos.X, TargetPos.Y, TargetPos.Z);
		}
		return;
	}

	/** Inner zone(Resimulation with full forward prediction)
	* - AlignedTime <= 0 - LOD wants to align this particle 0ms behind the current (forward predicted) timeline */
	if (LodData->AlignedTime <= 0.0f)
	{
		Target.RepMode = EPhysicsReplicationMode::Resimulation;
		Target.SimDecayTimeScale = 1.0f;
		Target.LastLODFrame = CurrentFrame;

		if (PhysicsReplicationLODCVars::bTransitionModeDebugLog)
		{
			UE_LOGF(LogPhysics, Log, "[ReplicationLOD] InnerZone | CurrentFrame=%d AlignedFrame=%d AlignedTime=%.4f | RepMode=Resimulation"
				, CurrentFrame, LodData->AlignedFrame, LodData->AlignedTime);
		}
		return;
	}

	// Transition zone, choose between Extrapolation (PI + extrapolated target) and ResimDecay (Resimulation + SimulationDecay) based on a fraction-of-prediction threshold

	// Number of frames closest to the latest received server state that use Extrapolation; the remainder of the transition zone uses ResimDecay
	const int32 ExtrapFramesThreshold = FMath::Max(
		PhysicsReplicationLODCVars::TransitionExtrapFrameMin,
		FMath::CeilToInt(PhysicsReplicationLODCVars::TransitionExtrapFraction * FullPredictionFrames));
	const int32 ExtrapolationFrameBoundary = LocalFrame + ExtrapFramesThreshold;

	Target.LastLODFrame = CurrentFrame;

	// Apply Extrapolation
	if (LodData->AlignedFrame <= ExtrapolationFrameBoundary)
	{
		Target.RepMode = EPhysicsReplicationMode::PredictiveInterpolation;
		Target.SimDecayTimeScale = 1.0f;

		// Run target extrapolation for non-sleeping particle
		const bool bShouldSleep = (Target.TargetState.Flags & ERigidBodyFlags::Sleeping) != 0;
		const FVector ServerTargetPos = Target.TargetState.Position; // Pre-extrapolation snapshot for logs.
		if (!bShouldSleep)
		{
			const float AlignedPredictionTime = FullPredictionTime - LodData->AlignedTime;
			FPhysicsReplicationAsync::ExtrapolateTarget(Target, AlignedPredictionTime);
			Target.TickCount = LodData->AlignedFrame - LocalFrame;
		}

		if (PhysicsReplicationLODCVars::bTransitionModeDebugLog)
		{
			Chaos::FReadPhysicsObjectInterface_Internal ReadInterface = Chaos::FPhysicsObjectInternalInterface::GetRead();
			const Chaos::FGeometryParticleHandle* DebugHandle = ReadInterface.GetParticle(PhysicsObjectHandle);
			const FVector ParticlePos = DebugHandle ? DebugHandle->GetX() : FVector::ZeroVector;
			const FVector ExtrapolatedTargetPos = Target.TargetState.Position;
			const TCHAR* Tag = bShouldSleep ? TEXT("TransitionExtrapolationSleep") : TEXT("TransitionExtrapolation");
			UE_LOGF(LogPhysics, Log, "[ReplicationLOD] %ls | CurrentFrame=%d LocalFrame=%d AlignedFrame=%d | FullPredictionFrames=%d FullPredictionTime=%.4f AlignedTime=%.4f | AlignDistance=%d ExtrapFramesThreshold=%d | ParticlePos=(%.2f,%.2f,%.2f) ServerTargetPos=(%.2f,%.2f,%.2f) ExtrapolatedTargetPos=(%.2f,%.2f,%.2f) | RepMode=PredictiveInterpolation"
				, Tag
				, CurrentFrame, LocalFrame, LodData->AlignedFrame
				, FullPredictionFrames, FullPredictionTime, LodData->AlignedTime
				, AlignDistance, ExtrapFramesThreshold
				, ParticlePos.X, ParticlePos.Y, ParticlePos.Z
				, ServerTargetPos.X, ServerTargetPos.Y, ServerTargetPos.Z
				, ExtrapolatedTargetPos.X, ExtrapolatedTargetPos.Y, ExtrapolatedTargetPos.Z);
		}
		return;
	}

	// Apply ResimDecay
	Target.RepMode = EPhysicsReplicationMode::Resimulation;

	// Calculate TimeScale to decay the simulation with during resim to make the particle end up at AlignedTime at the end of a resim
	Target.SimDecayTimeScale = FMath::Clamp(1.0f - (LodData->AlignedTime / FullPredictionTime), 0.0f, 1.0f);
	
	if (Chaos::FRewindData* RewindData = GetOrEnableRewindData(RigidsSolver))
	{
		// Request a resimulation for particles while inside the transition ZoneResim
		Chaos::FReadPhysicsObjectInterface_Internal Interface = Chaos::FPhysicsObjectInternalInterface::GetRead();
		Chaos::FPBDRigidParticleHandle* RigidParticle = Interface.GetRigidParticle(PhysicsObjectHandle);
		RewindData->RequestResimulation(LocalFrame, RigidParticle);
	}

	if (PhysicsReplicationLODCVars::bTransitionModeDebugLog)
	{
		Chaos::FReadPhysicsObjectInterface_Internal ReadInterface = Chaos::FPhysicsObjectInternalInterface::GetRead();
		const Chaos::FGeometryParticleHandle* DebugHandle = ReadInterface.GetParticle(PhysicsObjectHandle);
		const FVector ParticlePos = DebugHandle ? DebugHandle->GetX() : FVector::ZeroVector;
		const FVector ServerTargetPos = Target.TargetState.Position;
		UE_LOGF(LogPhysics, Log, "[ReplicationLOD] TransitionResim | CurrentFrame=%d LocalFrame=%d AlignedFrame=%d | FullPredictionFrames=%d FullPredictionTime=%.4f AlignedTime=%.4f | AlignDistance=%d ExtrapFramesThreshold=%d | SimDecayTimeScale=%.4f | ParticlePos=(%.2f,%.2f,%.2f) ServerTargetPos=(%.2f,%.2f,%.2f) | RepMode=Resimulation"
			, CurrentFrame, LocalFrame, LodData->AlignedFrame
			, FullPredictionFrames, FullPredictionTime, LodData->AlignedTime
			, AlignDistance, ExtrapFramesThreshold, Target.SimDecayTimeScale
			, ParticlePos.X, ParticlePos.Y, ParticlePos.Z
			, ServerTargetPos.X, ServerTargetPos.Y, ServerTargetPos.Z);
	}
}

//** Async function for legacy replication flow that goes partially through GT to then finishes in PT in this function. */
void FPhysicsReplicationAsync::DefaultReplication_DEPRECATED(Chaos::FRigidBodyHandle_Internal* Handle, const FPhysicsRepAsyncInputData& State, const float DeltaSeconds, const FPhysicsRepErrorCorrectionData& ErrorCorrection)
{
	if (Handle && Handle->CanTreatAsRigid())
	{
		const float LinearVelocityCoefficient = ErrorCorrection.LinearVelocityCoefficient;
		const float AngularVelocityCoefficient = ErrorCorrection.AngularVelocityCoefficient;
		const float PositionLerp = ErrorCorrection.PositionLerp;
		const float AngleLerp = ErrorCorrection.AngleLerp;

		const FVector TargetPos = State.TargetState.Position;
		const FQuat TargetQuat = State.TargetState.Quaternion;

		// Get Current state
		FRigidBodyState CurrentState;
		CurrentState.Position = Handle->X();
		CurrentState.Quaternion = Handle->R();
		CurrentState.AngVel = Handle->W();
		CurrentState.LinVel = Handle->V();

		FVector LinDiff;
		float LinDiffSize;
		FVector AngDiffAxis;
		float AngDiff;
		float AngDiffSize;
		ComputeDeltas(CurrentState.Position, CurrentState.Quaternion, TargetPos, TargetQuat, LinDiff, LinDiffSize, AngDiffAxis, AngDiff, AngDiffSize);

		const FVector NewLinVel = FVector(State.TargetState.LinVel) + (LinDiff * LinearVelocityCoefficient * DeltaSeconds);
		const FVector NewAngVel = FVector(State.TargetState.AngVel) + (AngDiffAxis * AngDiff * AngularVelocityCoefficient * DeltaSeconds);

		const FVector NewPos = FMath::Lerp(FVector(CurrentState.Position), TargetPos, PositionLerp);
		const FQuat NewAng = FQuat::Slerp(CurrentState.Quaternion, TargetQuat, AngleLerp);

		Handle->SetX(NewPos);
		Handle->SetR(NewAng);
		Handle->SetV(NewLinVel);
		Handle->SetW(FMath::DegreesToRadians(NewAngVel));

		if (State.TargetState.Flags & ERigidBodyFlags::Sleeping)
		{
			// don't allow kinematic to sleeping transition
			if (Handle->ObjectState() != Chaos::EObjectStateType::Kinematic)
			{
				Chaos::FPBDRigidsSolver* RigidsSolver = static_cast<Chaos::FPBDRigidsSolver*>(GetSolver());
				if (RigidsSolver)
				{
					RigidsSolver->GetEvolution()->SetParticleObjectState(Handle->GetProxy()->GetHandle_LowLevel()->CastToRigidParticle(), Chaos::EObjectStateType::Sleeping);	//todo: move object state into physics thread api
				}
			}
		}
	}
}


/** Default replication, run in simulation tick */
bool FPhysicsReplicationAsync::DefaultReplication(Chaos::FPBDRigidParticleHandle* Handle, FReplicatedPhysicsTargetAsync& Target, const float DeltaSeconds)
{
	Chaos::FPBDRigidsSolver* RigidsSolver = static_cast<Chaos::FPBDRigidsSolver*>(GetSolver());
	if (RigidsSolver == nullptr)
	{
		return true;
	}

	if (PhysicsReplicationCVars::ResimulationCVars::bDisableReplicationOnInteraction && ParticlesInResimIslands.Contains(Handle->GetHandleIdx()))
	{
		return false;
	}

	//
	// NOTES:
	//
	// The operation of this method has changed since 4.18.
	//
	// When a new remote physics state is received, this method will
	// be called on tick until the local state is within an adequate
	// tolerance of the new state.
	//
	// The received state is extrapolated based on ping, by some
	// adjustable amount.
	//
	// A correction velocity is added new state's velocity, and assigned
	// to the body. The correction velocity scales with the positional
	// difference, so without the interference of external forces, this
	// will result in an exponentially decaying correction.
	//
	// Generally it is not needed and will interrupt smoothness of
	// the replication, but stronger corrections can be obtained by
	// adjusting position lerping.
	//
	// If progress is not being made towards equilibrium, due to some
	// divergence in physics states between the owning and local sims,
	// an error value is accumulated, representing the amount of time
	// spent in an unresolvable state.
	//
	// Once the error value has exceeded some threshold (0.5 seconds
	// by default), a hard snap to the target physics state is applied.
	//


	bool bRestoredState = true;
	const FRigidBodyState NewState = Target.TargetState;
	const float NewQuatSizeSqr = NewState.Quaternion.SizeSquared();


	const FString ObjectName
#if CHAOS_DEBUG_NAME
		= Handle && Handle->DebugName() ? *Handle->DebugName() : FString();
#else
		= FString();
#endif

	// failure cases
	if (Handle == nullptr)
	{
		UE_LOGF(LogPhysics, Warning, "Trying to replicate rigid state for non-rigid particle. (%ls)", *ObjectName);
		return bRestoredState;
	}
	else if (NewQuatSizeSqr < UE_KINDA_SMALL_NUMBER)
	{
		UE_LOGF(LogPhysics, Warning, "Invalid zero quaternion set for body. (%ls)", *ObjectName);
		return bRestoredState;
	}
	else if (FMath::Abs(NewQuatSizeSqr - 1.f) > UE_KINDA_SMALL_NUMBER)
	{
		UE_LOGF(LogPhysics, Warning, "Quaternion (%f %f %f %f) with non-unit magnitude detected. (%ls)",
			NewState.Quaternion.X, NewState.Quaternion.Y, NewState.Quaternion.Z, NewState.Quaternion.W, *ObjectName);
		return bRestoredState;
	}
	// Grab configuration variables from engine config or from CVars if overriding is turned on.
	const float NetPingExtrapolation = CharacterMovementCVars::NetPingExtrapolation >= 0.0f ? CharacterMovementCVars::NetPingExtrapolation : ErrorCorrectionDefault.PingExtrapolation;
	const float NetPingLimit = CharacterMovementCVars::NetPingLimit > 0.0f ? CharacterMovementCVars::NetPingLimit : ErrorCorrectionDefault.PingLimit;
	const float ErrorPerLinearDiff = CharacterMovementCVars::ErrorPerLinearDifference >= 0.0f ? CharacterMovementCVars::ErrorPerLinearDifference : ErrorCorrectionDefault.ErrorPerLinearDifference;
	const float ErrorPerAngularDiff = CharacterMovementCVars::ErrorPerAngularDifference >= 0.0f ? CharacterMovementCVars::ErrorPerAngularDifference : ErrorCorrectionDefault.ErrorPerAngularDifference;
	const float MaxRestoredStateError = CharacterMovementCVars::MaxRestoredStateError >= 0.0f ? CharacterMovementCVars::MaxRestoredStateError : ErrorCorrectionDefault.MaxRestoredStateError;
	const float ErrorAccumulationSeconds = CharacterMovementCVars::ErrorAccumulationSeconds >= 0.0f ? CharacterMovementCVars::ErrorAccumulationSeconds : ErrorCorrectionDefault.ErrorAccumulationSeconds;
	const float ErrorAccumulationDistanceSq = CharacterMovementCVars::ErrorAccumulationDistanceSq >= 0.0f ? CharacterMovementCVars::ErrorAccumulationDistanceSq : ErrorCorrectionDefault.ErrorAccumulationDistanceSq;
	const float ErrorAccumulationSimilarity = CharacterMovementCVars::ErrorAccumulationSimilarity >= 0.0f ? CharacterMovementCVars::ErrorAccumulationSimilarity : ErrorCorrectionDefault.ErrorAccumulationSimilarity;
	const float PositionLerp = CharacterMovementCVars::PositionLerp >= 0.0f ? CharacterMovementCVars::PositionLerp : ErrorCorrectionDefault.PositionLerp;
	const float LinearVelocityCoefficient = CharacterMovementCVars::LinearVelocityCoefficient >= 0.0f ? CharacterMovementCVars::LinearVelocityCoefficient : ErrorCorrectionDefault.LinearVelocityCoefficient;
	const float AngleLerp = CharacterMovementCVars::AngleLerp >= 0.0f ? CharacterMovementCVars::AngleLerp : ErrorCorrectionDefault.AngleLerp;
	const float AngularVelocityCoefficient = CharacterMovementCVars::AngularVelocityCoefficient >= 0.0f ? CharacterMovementCVars::AngularVelocityCoefficient : ErrorCorrectionDefault.AngularVelocityCoefficient;
	float MaxLinearHardSnapDistance = CharacterMovementCVars::MaxLinearHardSnapDistance >= 0.f ? CharacterMovementCVars::MaxLinearHardSnapDistance : ErrorCorrectionDefault.MaxLinearHardSnapDistance;
	MaxLinearHardSnapDistance = SettingsCurrent.DefaultReplicationSettings.GetMaxLinearHardSnapDistance(MaxLinearHardSnapDistance);


	// Get Current state
	FRigidBodyState CurrentState;
	CurrentState.Position = Handle->GetX();
	CurrentState.Quaternion = Handle->GetR();
	CurrentState.AngVel = Handle->GetW();
	CurrentState.LinVel = Handle->GetV();


	// Starting from the last known authoritative position, and
	// extrapolate an approximation using the last known velocity
	// and ping.
	const float PingSeconds = FMath::Clamp(LatencyOneWay, 0.f, NetPingLimit);
	const float ExtrapolationDeltaSeconds = PingSeconds * NetPingExtrapolation;
	const FVector ExtrapolationDeltaPos = NewState.LinVel * ExtrapolationDeltaSeconds;
	const FVector_NetQuantize100 TargetPos = NewState.Position + ExtrapolationDeltaPos;
	float NewStateAngVel;
	FVector NewStateAngVelAxis;
	NewState.AngVel.FVector::ToDirectionAndLength(NewStateAngVelAxis, NewStateAngVel);
	NewStateAngVel = FMath::DegreesToRadians(NewStateAngVel);
	const FQuat ExtrapolationDeltaQuaternion = FQuat(NewStateAngVelAxis, NewStateAngVel * ExtrapolationDeltaSeconds);
	FQuat TargetQuat = ExtrapolationDeltaQuaternion * NewState.Quaternion;


	FVector LinDiff;
	float LinDiffSize;
	FVector AngDiffAxis;
	float AngDiff;
	float AngDiffSize;
	ComputeDeltas(CurrentState.Position, CurrentState.Quaternion, TargetPos, TargetQuat, LinDiff, LinDiffSize, AngDiffAxis, AngDiff, AngDiffSize);

	/////// ACCUMULATE ERROR IF NOT APPROACHING SOLUTION ///////

	// Store sleeping state
	const bool bShouldSleep = (NewState.Flags & ERigidBodyFlags::Sleeping) != 0;
	const bool bWasAwake = !Handle->Sleeping();
	const bool bAutoWake = false;

	const float Error = (LinDiffSize * ErrorPerLinearDiff) + (AngDiffSize * ErrorPerAngularDiff);

	bRestoredState = Error < MaxRestoredStateError;
	if (bRestoredState)
	{
		Target.AccumulatedErrorSeconds = 0.0f;
	}
	else
	{
		//
		// The heuristic for error accumulation here is:

		// 1. Did the physics tick from the previous step fail to
		//    move the body towards a resolved position?
		// 2. Was the linear error in the same direction as the
		//    previous frame?
		// 3. Is the linear error large enough to accumulate error?
		//
		// If these conditions are met, then "error" time will accumulate.
		// Once error has accumulated for a certain number of seconds,
		// a hard-snap to the target will be performed.
		//
		// TODO: Rotation while moving linearly can still mess up this
		// heuristic. We need to account for it.
		//

		// Project the change in position from the previous tick onto the
		// linear error from the previous tick. This value roughly represents
		// how much correction was performed over the previous physics tick.
		const float PrevProgress = FVector::DotProduct(
			FVector(CurrentState.Position) - Target.PrevPos,
			(Target.PrevPosTarget - Target.PrevPos).GetSafeNormal());

		// Project the current linear error onto the linear error from the
		// previous tick. This value roughly represents how little the direction
		// of the linear error state has changed, and how big the error is.
		const float PrevSimilarity = FVector::DotProduct(
			TargetPos - FVector(CurrentState.Position),
			Target.PrevPosTarget - Target.PrevPos);

		// If the conditions from the heuristic outlined above are met, accumulate
		// error. Otherwise, reduce it.
		if (PrevProgress < ErrorAccumulationDistanceSq &&
			PrevSimilarity > ErrorAccumulationSimilarity)
		{
			Target.AccumulatedErrorSeconds += DeltaSeconds;
		}
		else
		{
			Target.AccumulatedErrorSeconds = FMath::Max(Target.AccumulatedErrorSeconds - DeltaSeconds, 0.0f);
		}

		// Hard snap if error accumulation or linear error is big enough, and clear the error accumulator.
		const bool bHardSnap =
			LinDiffSize > MaxLinearHardSnapDistance ||
			Target.AccumulatedErrorSeconds > ErrorAccumulationSeconds ||
			CharacterMovementCVars::AlwaysHardSnap;

		if (bHardSnap)
		{
#if !UE_BUILD_SHIPPING
			if (PhysicsReplicationCVars::LogPhysicsReplicationHardSnaps)
			{
				UE_LOGF(LogTemp, Warning, "Simulated HARD SNAP - \nCurrent Pos - %ls, Target Pos - %ls\n CurrentState.LinVel - %ls, New Lin Vel - %ls\nTarget Extrapolation Delta - %ls, Is Asleep - %d, Prev Progress - %f, Prev Similarity - %f",
					*CurrentState.Position.ToString(), *TargetPos.ToString(), *CurrentState.LinVel.ToString(), *NewState.LinVel.ToString(),
					*ExtrapolationDeltaPos.ToString(), Handle->Sleeping(), PrevProgress, PrevSimilarity);

				if (LinDiffSize > MaxLinearHardSnapDistance)
				{
					UE_LOGF(LogTemp, Warning, "Hard snap due to linear difference error");
				}
				else
				{
					UE_LOGF(LogTemp, Warning, "Hard snap due to accumulated error")
				}
			}
#endif
			// Too much error so just snap state here and be done with it
			Target.AccumulatedErrorSeconds = 0.0f;
			bRestoredState = true;

			// Set XRVW to hard snap dynamic object and force recalculation of friction
			const bool bCorrectConnectedBodies = SettingsCurrent.DefaultReplicationSettings.GetCorrectConnectedBodies();
			const bool bCorrectConnectedBodiesFriction = SettingsCurrent.DefaultReplicationSettings.GetCorrectConnectedBodiesFriction();
			RigidsSolver->GetEvolution()->ApplyParticleTransformCorrection(Handle, TargetPos, TargetQuat, bCorrectConnectedBodies, bCorrectConnectedBodiesFriction, ReplicatedParticleIDs);
			Handle->SetV(NewState.LinVel);
			Handle->SetW(FMath::DegreesToRadians(NewState.AngVel));
		}
		else
		{
			const FVector NewLinVel = FVector(Target.TargetState.LinVel) + (LinDiff * LinearVelocityCoefficient * DeltaSeconds);
			const FVector NewAngVel = FVector(Target.TargetState.AngVel) + (AngDiffAxis * AngDiff * AngularVelocityCoefficient * DeltaSeconds);

			const FVector NewPos = FMath::Lerp(FVector(CurrentState.Position), TargetPos, PositionLerp);
			const FQuat NewAng = FQuat::Slerp(CurrentState.Quaternion, TargetQuat, AngleLerp);

			Handle->SetX(NewPos);
			Handle->SetR(NewAng);
			Handle->SetV(NewLinVel);
			Handle->SetW(FMath::DegreesToRadians(NewAngVel));
		}
	}

	if (bShouldSleep)
	{
		// don't allow kinematic to sleeping transition
		if (Handle->ObjectState() != Chaos::EObjectStateType::Kinematic)
		{
			RigidsSolver->GetEvolution()->SetParticleObjectState(Handle, Chaos::EObjectStateType::Sleeping);
		}
	}

	Target.PrevPosTarget = TargetPos;
	Target.PrevPos = FVector(CurrentState.Position);

	return bRestoredState;
}

/** Interpolating towards replicated states from the server while predicting local physics 
* TODO, detailed description
*/
bool FPhysicsReplicationAsync::PredictiveInterpolation(Chaos::FPBDRigidParticleHandle* Handle, FReplicatedPhysicsTargetAsync& Target, const float DeltaSeconds)
{
	if (PhysicsReplicationCVars::PredictiveInterpolationCVars::bSkipReplication)
	{
		return true;
	}

	Chaos::FPBDRigidsSolver* RigidsSolver = static_cast<Chaos::FPBDRigidsSolver*>(GetSolver());
	if (RigidsSolver == nullptr)
	{
		return true;
	}

	if (PhysicsReplicationCVars::ResimulationCVars::bDisableReplicationOnInteraction && ParticlesInResimIslands.Contains(Handle->GetHandleIdx()))
	{
		// If particle is in an island with a resim object, don't run replication and wait for an up to date target (after leaving the island)
		Target.SetWaiting(RigidsSolver->GetCurrentFrame() + NetworkPhysicsTickOffset, EPhysicsReplicationMode::Resimulation);
		return false;
	}

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	if (PhysicsReplicationCVars::PredictiveInterpolationCVars::bDrawDebugTargets)
	{
		const FVector Offset = FVector(0.0f, 0.0f, PhysicsReplicationCVars::PredictiveInterpolationCVars::DrawDebugZOffset);
		const FVector Pos = Target.TargetState.Position + Offset;
		const int32 SizeMultiplier = FMath::Clamp(Target.TickCount, -4, 30);
		Chaos::FDebugDrawQueue::GetInstance().DrawDebugBox(Pos, FVector(5.0f + SizeMultiplier * 0.75f, 5.0f + SizeMultiplier * 0.75f, 5.0f + SizeMultiplier * 0.75f), Target.TargetState.Quaternion, FColor::MakeRandomSeededColor(Target.ServerFrame), false, PhysicsReplicationCVars::DebugDrawLifeTime, 0, 1.0f);
	}
#endif

	const bool bIsSleeping = Handle->IsSleeping();
	const bool bCanSimulate = Handle->IsDynamic() || bIsSleeping;

	// Accumulate sleep time or reset back to 0s if not sleeping
	Target.AccumulatedSleepSeconds = bIsSleeping ? (Target.AccumulatedSleepSeconds + DeltaSeconds) : 0.0f;
	
	// Helper for sleep and target clearing at replication end
	auto EndReplicationHelper = [RigidsSolver, Handle, bCanSimulate, bIsSleeping, DeltaSeconds](FReplicatedPhysicsTargetAsync& Target, bool bOkToClear) -> bool
	{
		const bool bShouldSleep = (Target.TargetState.Flags & ERigidBodyFlags::Sleeping) != 0;
		const bool bReplicatingPhysics = (Target.TargetState.Flags & ERigidBodyFlags::RepPhysics) != 0;

		// --- Set Sleep State ---
		if (bOkToClear && bShouldSleep && bCanSimulate)
		{
			RigidsSolver->GetEvolution()->SetParticleObjectState(Handle, Chaos::EObjectStateType::Sleeping);
			if (PhysicsReplicationCVars::PredictiveInterpolationCVars::bSleepConnectedBodies)
			{
				RigidsSolver->GetEvolution()->ApplySleepOnConnectedParticles(Handle);
			}
		}

		// --- Should replication stop? ---
		const bool bClearTarget =
			((bOkToClear && bShouldSleep && Target.AccumulatedSleepSeconds >= PhysicsReplicationCVars::PredictiveInterpolationCVars::SleepSecondsClearTarget) // Allow clearing the target due to sleeping after the object has been sleeping for n seconds
			|| (bOkToClear && !bReplicatingPhysics) // If replication say it's okay to clear the target and the object shouldn't replicate physics anymore, clear the target
			|| (bOkToClear && !bCanSimulate)) // If replication say it's okay to clear the target and the object can't simulate, clear the target
			&& !PhysicsReplicationCVars::PredictiveInterpolationCVars::bDontClearTarget;

		// --- Target Prediction ---
		if (!bClearTarget && Target.bAllowTargetAltering)
		{
			const int32 ExtrapolationTickLimit = FMath::Max(
				FMath::CeilToInt(Target.AverageReceiveInterval * PhysicsReplicationCVars::PredictiveInterpolationCVars::ExtrapolationTimeMultiplier), // Extrapolate time based on receive interval * multiplier
				FMath::CeilToInt(PhysicsReplicationCVars::PredictiveInterpolationCVars::ExtrapolationMinTime / DeltaSeconds)); // At least extrapolate for N seconds

			if (Target.TickCount <= ExtrapolationTickLimit)
			{
				FPhysicsReplicationAsync::ExtrapolateTarget(Target, 1, DeltaSeconds);
			}
			else
			{
				// If we reach the extrapolation limit, disable target from being altered
				Target.bAllowTargetAltering = false;
			}
		}

		return bClearTarget;
	};

	// If waiting on an up to date state, early out but allow target clearing since we might not receive a new state if target is already set to sleep for example
	if (Target.IsWaiting())
	{
		return EndReplicationHelper(Target, true);
	}

	// If target velocity is low enough, check the distance from the current position to the source position of our target to see if it's low enough to early out of replication
	const bool bXCanEarlyOut = (PhysicsReplicationCVars::PredictiveInterpolationCVars::bEarlyOutWithVelocity || Target.TargetState.LinVel.SizeSquared() < UE_KINDA_SMALL_NUMBER) &&
		(Target.PrevPosTarget - Handle->GetX()).SizeSquared() < PhysicsReplicationCVars::PredictiveInterpolationCVars::EarlyOutDistanceSqr;

	// Early out if we are within range of target, also apply target sleep state
	if (bXCanEarlyOut)
	{
		// Get the rotational offset between the blended rotation target and the current rotation
		const FQuat TargetRotDelta = Target.TargetState.Quaternion * Handle->GetR().Inverse();

		// Convert to angle and axis
		float Angle;
		FVector Axis;
		TargetRotDelta.ToAxisAndAngle(Axis, Angle);
		Angle = FMath::RadiansToDegrees(FMath::UnwindRadians(Angle));
		Angle = FMath::Abs(Angle);

		if (Angle < PhysicsReplicationCVars::PredictiveInterpolationCVars::EarlyOutAngle)
		{
			// Early Out
			return EndReplicationHelper(Target, true);
		}
	}
	
	// Update the AverageReceiveInterval if Target.ReceiveInterval has a valid value to update from
	Target.AverageReceiveInterval = Target.ReceiveInterval == 0 ? Target.AverageReceiveInterval : FMath::Lerp(Target.AverageReceiveInterval, Target.ReceiveInterval, FMath::Clamp((1.0f / (Target.ReceiveInterval * PhysicsReplicationCVars::PredictiveInterpolationCVars::AverageReceiveIntervalSmoothing)), 0.0f, 1.0f));

	// CurrentState
	FRigidBodyState CurrentState;
	CurrentState.Position = Handle->GetX();
	CurrentState.Quaternion = Handle->GetR();
	CurrentState.LinVel = Handle->GetV();
	CurrentState.AngVel = Handle->GetW(); // Radians

	// NewState
	const FVector TargetPos = FVector(Target.TargetState.Position);
	const FQuat TargetRot = Target.TargetState.Quaternion;
	const FVector TargetLinVel = FVector(Target.TargetState.LinVel);
	const FVector TargetAngVel = FVector(FMath::DegreesToRadians(Target.TargetState.AngVel)); // Radians

	/** --- Reconciliation --- */
	const bool bHardSnap = (!bCanSimulate && PhysicsReplicationCVars::PredictiveInterpolationCVars::bKinematicHardSnap)
		|| Target.AccumulatedErrorSeconds > PhysicsReplicationCVars::PredictiveInterpolationCVars::ErrorAccumulationSeconds
		|| PhysicsReplicationCVars::PredictiveInterpolationCVars::bAlwaysHardSnap;

	if (bHardSnap)
	{
		Target.AccumulatedErrorSeconds = 0.0f;

		if (Handle->IsKinematic())
		{
			// Set a FKinematicTarget to hard snap kinematic object
			const Chaos::FKinematicTarget KinTarget = Chaos::FKinematicTarget::MakePositionTarget(Target.PrevPosTarget, Target.PrevRotTarget); // Uses EKinematicTargetMode::Position
			RigidsSolver->GetEvolution()->SetParticleKinematicTarget(Handle, KinTarget);
		}
		else 
		{
			// Set XRVW to hard snap dynamic object and force recalculation of friction
			const bool bCorrectConnectedBodies = SettingsCurrent.PredictiveInterpolationSettings.GetCorrectConnectedBodies();
			RigidsSolver->GetEvolution()->ApplyParticleTransformCorrection(Handle, Target.PrevPosTarget, Target.PrevRotTarget, bCorrectConnectedBodies, /*bInRecalculateFrictionOnConnectedBodies*/ true, ReplicatedParticleIDs);
			Handle->SetV(TargetLinVel);
			Handle->SetW(TargetAngVel);
		}

		// Cache data for next replication
		Target.PrevLinVel = FVector(Target.TargetState.LinVel);

		// End replication and go to sleep if that's requested
		return EndReplicationHelper(Target, true);
	}

	/** If target velocities are low enough, check the traveled direction and distance from previous frame and compare with replicated linear velocity.
	* If the object isn't moving enough along the replicated velocity it's considered stuck and needs reconciliation.
	* SoftSnap is performed each tick while there is a registered error, if enough time pass HardSnap forces the object into the correct state. */
	bool bSoftSnap = false;

	if (PhysicsReplicationCVars::PredictiveInterpolationCVars::bDisableErrorVelocityLimits ||
		(TargetLinVel.Size() < PhysicsReplicationCVars::PredictiveInterpolationCVars::ErrorAccLinVelMaxLimit && TargetAngVel.Size() < PhysicsReplicationCVars::PredictiveInterpolationCVars::ErrorAccAngVelMaxLimit))
	{
		const FVector PrevDiff = CurrentState.Position - Target.PrevPos;
		const float ExpectedDistance = (Target.PrevLinVel * DeltaSeconds).Size();
		const float CoveredDistance = FVector::DotProduct(PrevDiff, Target.PrevLinVel.GetSafeNormal());
		const float CoveredAplha = FMath::Clamp(CoveredDistance / ExpectedDistance, 0.0f, 1.0f);

		// If the object is moving less than X% of the expected distance, accumulate error seconds
		if (CoveredAplha < PhysicsReplicationCVars::PredictiveInterpolationCVars::MinExpectedDistanceCovered)
		{
			Target.AccumulatedErrorSeconds += DeltaSeconds;
			bSoftSnap = true;
		}
		else if (Target.AccumulatedErrorSeconds > 0.f)
		{
			const float DecreaseTime = DeltaSeconds * PhysicsReplicationCVars::PredictiveInterpolationCVars::ErrorAccumulationDecreaseMultiplier;
			Target.AccumulatedErrorSeconds = FMath::Max(Target.AccumulatedErrorSeconds - DecreaseTime, 0.0f);
			bSoftSnap = true;
		}
	}
	else
	{
		Target.AccumulatedErrorSeconds = 0;
	}

	if (SettingsCurrent.PredictiveInterpolationSettings.GetDisableSoftSnap())
	{
		bSoftSnap = false;
	}

	if (Handle->IsKinematic()) // Smooth Kinematic Replication
	{
		const bool bKinematicPrediction = PhysicsReplicationCVars::PredictiveInterpolationCVars::bKinematicPrediction;
		const float InterpolationTicks = FMath::CeilToInt(Target.AverageReceiveInterval) - (RigidsSolver->GetCurrentFrame() - Target.ReceiveFrame);

		if ((bKinematicPrediction && Target.bAllowTargetAltering) || InterpolationTicks > 0)
		{
			/* Calculate the Lerp value for a smooth interpolation
			* ------------------------------------------------------------------------------
			* bKinematicPrediction is True :: Interpolate towards the target that gets forward predicted each tick
			*	1 / 4 = 0.25 = 25% interpolation each time (if AverageReceiveInterval is 4)
			* ------------------------------------------------------------------------------
			* bKinematicPrediction is False :: Interpolate from current position to the static source for the current target, we need to cover the same amount of distance but from a decaying distance
			*	| ---> | ------------------ |
			*	0%    25%				   100%		(1 / 4 = 0.25)
			*		   | ---> | ----------- |
			*		   0%	 33%		   100%		(1 / 3 = 0.33)
			*				  | ---> | ---- |
			*				  0%    50%    100%		(1 / 2 = 0.5)
			*						 | ---> |
			*						 0%    100%		(1 / 1 = 1.0)
			* ------------------------------------------------------------------------------
			*/
			const float Lerp = 1.f / (bKinematicPrediction ? Target.AverageReceiveInterval : InterpolationTicks);
			
			// Interpolate position and rotation from current position towards target position based on either predicted target or source target
			const FVector KinTargetPos = FMath::Lerp(CurrentState.Position,
				(bKinematicPrediction ? Target.TargetState.Position : Target.PrevPosTarget),
				Lerp);
			const FQuat KinTargetRot = FQuat::Slerp(CurrentState.Quaternion,
				(bKinematicPrediction ? Target.TargetState.Quaternion : Target.PrevRotTarget),
				Lerp);

			// Apply kinematic target
			const Chaos::FKinematicTarget KinTarget = Chaos::FKinematicTarget::MakePositionTarget(KinTargetPos, KinTargetRot); // Uses EKinematicTargetMode::Position
			RigidsSolver->GetEvolution()->SetParticleKinematicTarget(Handle, KinTarget);

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
			if (PhysicsReplicationCVars::PredictiveInterpolationCVars::bDrawDebugTargets)
			{
				const FVector Offset = FVector(0.0f, 0.0f, PhysicsReplicationCVars::PredictiveInterpolationCVars::DrawDebugZOffset);
				const FVector Pos = KinTargetPos + Offset;
				const int32 SizeMultiplier = FMath::Clamp(Target.TickCount, -4, 30);
				Chaos::FDebugDrawQueue::GetInstance().DrawDebugSphere(Pos, 3.0f + SizeMultiplier * 0.75f, 8, FColor::MakeRandomSeededColor(Target.ServerFrame), false, PhysicsReplicationCVars::DebugDrawLifeTime, 0, 1.0f);
			}
#endif
		}
		else
		{
			// End replication and allow to clear target
			return EndReplicationHelper(Target, true);
		}
	}
	else // Dynamic Replication
	{
		// Wake up if sleeping
		if (bIsSleeping)
		{
			RigidsSolver->GetEvolution()->SetParticleObjectState(Handle, Chaos::EObjectStateType::Dynamic);
		}

		// Velocity-based Replication
		if (SettingsCurrent.PredictiveInterpolationSettings.GetVelocityBased())
		{
			// Calculate interpolation time based on current average receive rate
			const float AverageReceiveIntervalSeconds = Target.AverageReceiveInterval * DeltaSeconds;
			const float InterpolationTime = AverageReceiveIntervalSeconds * SettingsCurrent.PredictiveInterpolationSettings.GetPosInterpolationTimeMultiplier();

			// Calculate position correction time based on current Round Trip Time
			const float RTT = LatencyOneWay * 2.f;
			const float PosCorrectionTime = FMath::Max(SettingsCurrent.PredictiveInterpolationSettings.GetPosCorrectionTimeBase() + AverageReceiveIntervalSeconds + RTT * SettingsCurrent.PredictiveInterpolationSettings.GetPosCorrectionTimeMultiplier(),
				DeltaSeconds + SettingsCurrent.PredictiveInterpolationSettings.GetPosCorrectionTimeMin());
			const float RotCorrectionTime = FMath::Max(SettingsCurrent.PredictiveInterpolationSettings.GetRotCorrectionTimeBase() + AverageReceiveIntervalSeconds + RTT * SettingsCurrent.PredictiveInterpolationSettings.GetRotCorrectionTimeMultiplier(),
				DeltaSeconds + SettingsCurrent.PredictiveInterpolationSettings.GetRotCorrectionTimeMin());

			FVector CorrectionX = CurrentState.Position;
			if ((bXCanEarlyOut && SettingsCurrent.PredictiveInterpolationSettings.GetSkipVelocityRepOnPosEarlyOut()) == false)
			{	// --- Velocity Replication ---

				// Get PosDiff
				const FVector PosDiff = TargetPos - CurrentState.Position;

				// Get LinVelDiff by adding inverted CurrentState.LinVel to TargetLinVel
				const FVector LinVelDiff = -CurrentState.LinVel + TargetLinVel;

				// Calculate velocity blend amount for this tick as an alpha value
				const float VelocityAlpha = FMath::Clamp(DeltaSeconds / InterpolationTime, 0.0f, 1.0f);

				FVector RepLinVel;
				if (SettingsCurrent.PredictiveInterpolationSettings.GetCorrectionAsVelocity())
				{
					// Convert PosDiff to a velocity
					const FVector PosDiffVelocity = PosDiff / PosCorrectionTime;

					// Add PosDiffVelocity to LinVelDiff to get BlendedTargetVelocity
					const FVector BlendedTargetVelocity = LinVelDiff + PosDiffVelocity;

					// Add BlendedTargetVelocity onto current velocity
					RepLinVel = CurrentState.LinVel + (BlendedTargetVelocity * VelocityAlpha); // Same as (BlendedTargetVelocity / InterpolationTime) * DeltaSeconds
				}
				else // Positional correction as transform shift
				{
					// Add velocity diff onto current velocity
					RepLinVel = CurrentState.LinVel + (LinVelDiff * VelocityAlpha); // Same as (LinVelDiff / InterpolationTime) * DeltaSeconds

					// Calculate correction blend amount for this tick as an alpha value
					const float CorrectionAlpha = FMath::Clamp(DeltaSeconds / PosCorrectionTime, 0.0f, 1.0f);

					// Calculate the PosDiff amount to correct this tick
					const FVector PosDiffVelocityDelta = PosDiff * CorrectionAlpha; // Same as (PosDiff / PosCorrectionTime) * DeltaSeconds

					// The new position after correction
					CorrectionX = Handle->GetX() + PosDiffVelocityDelta;
				}

				// Apply velocity replication
				Handle->SetV(RepLinVel);

	#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
				if (PhysicsReplicationCVars::PredictiveInterpolationCVars::bDrawDebugVectors)
				{
					const FVector Offset = FVector(0.0f, 0.0f, PhysicsReplicationCVars::PredictiveInterpolationCVars::DrawDebugZOffset);
					const FVector OffsetAdd = FVector(0.0f, 0.0f, 10.0f);
					const FVector StartPos = TargetPos + Offset;
					FVector Direction = TargetLinVel;
					Direction.Normalize();
					Chaos::FDebugDrawQueue::GetInstance().DrawDebugDirectionalArrow((StartPos + OffsetAdd * 0), (StartPos + OffsetAdd * 0) + TargetLinVel * 0.5f, 5.0f, FColor::Green, false, -1.0f, 0, 2.0f);
					Chaos::FDebugDrawQueue::GetInstance().DrawDebugDirectionalArrow((StartPos + OffsetAdd * 1), (StartPos + OffsetAdd * 1) + CurrentState.LinVel * 0.5f, 5.0f, FColor::Blue, false, -1.0f, 0, 2.0f);
					Chaos::FDebugDrawQueue::GetInstance().DrawDebugDirectionalArrow((StartPos + OffsetAdd * 2), (StartPos + OffsetAdd * 2) + (Target.PrevLinVel - CurrentState.LinVel) * 0.5f, 5.0f, FColor::Red, false, -1.0f, 0, 2.0f);
					Chaos::FDebugDrawQueue::GetInstance().DrawDebugDirectionalArrow((StartPos + OffsetAdd * 3), (StartPos + OffsetAdd * 3) + RepLinVel * 0.5f, 5.0f, FColor::Magenta, false, -1.0f, 0, 2.0f);
					Chaos::FDebugDrawQueue::GetInstance().DrawDebugDirectionalArrow((StartPos + OffsetAdd * 4), (StartPos + OffsetAdd * 4) + (Target.PrevLinVel - RepLinVel) * 0.5f, 5.0f, FColor::Orange, false, -1.0f, 0, 2.0f);
					Chaos::FDebugDrawQueue::GetInstance().DrawDebugDirectionalArrow((StartPos + OffsetAdd * 5), (StartPos + OffsetAdd * 5) + Direction * RTT, 5.0f, FColor::White, false, -1.0f, 0, 2.0f);
					Chaos::FDebugDrawQueue::GetInstance().DrawDebugDirectionalArrow((StartPos + OffsetAdd * 6), (StartPos + OffsetAdd * 6) + Direction * InterpolationTime, 5.0f, FColor::Yellow, false, -1.0f, 0, 2.0f);
				}
	#endif
				// Cache data for next replication
				Target.PrevLinVel = FVector(RepLinVel);
			}

			FQuat CorrectionR = CurrentState.Quaternion;
			{	// --- Angular Velocity Replication ---

				// Get AngVelDiff by adding inverted CurrentState.AngVel to TargetAngVel
				const FVector AngVelDiff = -CurrentState.AngVel + TargetAngVel;

				// Calculate velocity blend amount for this tick as an alpha value
				const float VelocityAlpha = FMath::Clamp(DeltaSeconds / InterpolationTime, 0.0f, 1.0f);

				FVector RepAngVel;
				if (SettingsCurrent.PredictiveInterpolationSettings.GetCorrectionAsVelocity())
				{
					// Get RotDiff
					const FQuat RotDiff = TargetRot * CurrentState.Quaternion.Inverse();

					// Convert RotDiff to a velocity
					float WAngle;
					FVector WAxis;
					RotDiff.ToAxisAndAngle(WAxis, WAngle);
					WAngle = FMath::UnwindRadians(WAngle);
					const FVector RotDiffVelocity = FVector(WAxis * (WAngle / RotCorrectionTime));

					// Add RotDiffVelocity to AngVelDiff to get BlendedTargetVelocity
					const FVector BlendedTargetVelocity = AngVelDiff + RotDiffVelocity;

					// Add BlendedTargetVelocity to CurrentState.AngVel
					RepAngVel = CurrentState.AngVel + (BlendedTargetVelocity * VelocityAlpha); // Same as (BlendedTargetVelocity / InterpolationTime) * DeltaSeconds
				}
				else // Rotational correction as transform shift
				{
					// Add velocity diff onto current velocity
					RepAngVel = CurrentState.AngVel + (AngVelDiff * VelocityAlpha); // Same as (AngVelDiff / InterpolationTime) * DeltaSeconds

					// Calculate correction blend amount for this tick as an alpha value
					const float CorrectionAlpha = FMath::Clamp(DeltaSeconds / RotCorrectionTime, 0.0f, 1.0f);

					// The new position after correction
					CorrectionR = FQuat::Slerp(Handle->GetR(), TargetRot, CorrectionAlpha);
				}

				// Apply velocity replication
				Handle->SetW(RepAngVel);
			}

			// Cache data for next replication
			Target.PrevPos = FVector(CurrentState.Position);

			// Apply correction as a transform shift
			if (SettingsCurrent.PredictiveInterpolationSettings.GetCorrectionAsVelocity() == false)
			{	
				const bool bCorrectConnectedBodies = SettingsCurrent.PredictiveInterpolationSettings.GetCorrectConnectedBodies();
				const bool bCorrectConnectedBodiesFriction = SettingsCurrent.PredictiveInterpolationSettings.GetCorrectConnectedBodiesFriction();
				RigidsSolver->GetEvolution()->ApplyParticleTransformCorrection(Handle, CorrectionX, CorrectionR, bCorrectConnectedBodies, bCorrectConnectedBodiesFriction, ReplicatedParticleIDs);
			}
		}

		// Transform shift replication, if either soft-snap is needed or velocity-based replication is false
		if (bSoftSnap || SettingsCurrent.PredictiveInterpolationSettings.GetVelocityBased() == false)
		{
			const FVector SoftSnapPos = FMath::Lerp(FVector(CurrentState.Position),
				SettingsCurrent.PredictiveInterpolationSettings.GetSoftSnapToSource() ? Target.PrevPosTarget : Target.TargetState.Position,
				FMath::Clamp(SettingsCurrent.PredictiveInterpolationSettings.GetSoftSnapPosStrength(), 0.0f, 1.0f));

			const FQuat SoftSnapRot = FQuat::Slerp(CurrentState.Quaternion,
				SettingsCurrent.PredictiveInterpolationSettings.GetSoftSnapToSource() ? Target.PrevRotTarget : Target.TargetState.Quaternion,
				FMath::Clamp(SettingsCurrent.PredictiveInterpolationSettings.GetSoftSnapRotStrength(), 0.0f, 1.0f));

			// Apply correction as a transform shift
			const bool bCorrectConnectedBodies = SettingsCurrent.PredictiveInterpolationSettings.GetCorrectConnectedBodies();
			const bool bCorrectConnectedBodiesFriction = SettingsCurrent.PredictiveInterpolationSettings.GetCorrectConnectedBodiesFriction();
			RigidsSolver->GetEvolution()->ApplyParticleTransformCorrection(Handle, SoftSnapPos, SoftSnapRot, bCorrectConnectedBodies, bCorrectConnectedBodiesFriction, ReplicatedParticleIDs);
		}
	}

	return EndReplicationHelper(Target, false);
}

/** Static function to extrapolate a target for N ticks using X DeltaSeconds */
void FPhysicsReplicationAsync::ExtrapolateTarget(FReplicatedPhysicsTargetAsync& Target, const int32 ExtrapolateFrames, const float DeltaSeconds)
{
	const float ExtrapolationTime = DeltaSeconds * static_cast<float>(ExtrapolateFrames);
	FPhysicsReplicationAsync::ExtrapolateTarget(Target, ExtrapolationTime);
}

/** Static function to extrapolate a target for N Seconds */
void FPhysicsReplicationAsync::ExtrapolateTarget(FReplicatedPhysicsTargetAsync& Target, const float ExtrapolationTime)
{
	// Extrapolate target position
	Target.TargetState.Position = Target.TargetState.Position + Target.TargetState.LinVel * ExtrapolationTime;

	// Extrapolate target rotation
	float TargetAngVelSize;
	FVector TargetAngVelAxis;
	Target.TargetState.AngVel.FVector::ToDirectionAndLength(TargetAngVelAxis, TargetAngVelSize);
	TargetAngVelSize = FMath::DegreesToRadians(TargetAngVelSize);
	const FQuat TargetRotExtrapDelta = FQuat(TargetAngVelAxis, TargetAngVelSize * ExtrapolationTime);
	Target.TargetState.Quaternion = TargetRotExtrapDelta * Target.TargetState.Quaternion;
}

bool FPhysicsReplicationAsync::IsTargetValidForResim(const FReplicatedPhysicsTargetAsync& Target) const
{
	const Chaos::FPBDRigidsSolver* RigidsSolver = static_cast<const Chaos::FPBDRigidsSolver*>(GetSolver());
	if (RigidsSolver == nullptr)
	{
		return false;
	}

	const Chaos::FRewindData* RewindData = RigidsSolver->GetRewindData();
	if (RewindData == nullptr)
	{
		return false;
	}

	if (Target.ServerFrame <= 0)
	{
		return false;
	}

	const int32 LocalFrame = Target.ServerFrame - NetworkPhysicsTickOffset;

	if (!RewindData->IsFrameWithinRewindHistory(LocalFrame))
	{
		return false;
	}
	return true;
}

bool FPhysicsReplicationAsync::ShouldResimulateParticle(const Chaos::FPBDRigidParticleHandle* Handle, const FReplicatedPhysicsTargetAsync& Target, const float DeltaSeconds) const
{
	bool bShouldTriggerResim = false;
	if (IsTargetValidForResim(Target))
	{
		const Chaos::FRewindData* RewindData = static_cast<const Chaos::FPBDRigidsSolver*>(GetSolver())->GetRewindData();
		const int32 LocalFrame = Target.ServerFrame - NetworkPhysicsTickOffset;
		static constexpr Chaos::FFrameAndPhase::EParticleHistoryPhase RewindPhase = Chaos::FFrameAndPhase::EParticleHistoryPhase::PrePushData;

		// Get state from locally cached history for frame corresponding to received data
		const Chaos::FGeometryParticleState PastState = RewindData->GetPastStateAtFrame(*Handle, LocalFrame, RewindPhase);

		// Check which comparisons to perform to trigger resimulation from
		const bool bCompareX = Chaos::FPhysicsSolverBase::GetResimulationErrorPositionThresholdEnabled() || SettingsCurrent.ResimulationSettings.bOverrideResimulationErrorPositionThreshold;
		const bool bCompareR = Chaos::FPhysicsSolverBase::GetResimulationErrorRotationThresholdEnabled() || SettingsCurrent.ResimulationSettings.bOverrideResimulationErrorRotationThreshold;
		const bool bCompareV = Chaos::FPhysicsSolverBase::GetResimulationErrorLinearVelocityThresholdEnabled() || SettingsCurrent.ResimulationSettings.bOverrideResimulationErrorLinearVelocityThreshold;
		const bool bCompareW = Chaos::FPhysicsSolverBase::GetResimulationErrorAngularVelocityThresholdEnabled() || SettingsCurrent.ResimulationSettings.bOverrideResimulationErrorAngularVelocityThreshold;
		const bool bCompareSleep = PhysicsReplicationCVars::ResimulationCVars::bResimulateSleepDesync;

	#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
		// Debugging
		FColor DebugColor = FColor::Black;
		bool bResimV = false;
		bool bResimW = false;
	#endif

		// Check for positional discrepancy in Distance between client and server
		if (bCompareX)
		{
			const float ResimPositionErrorThreshold = SettingsCurrent.ResimulationSettings.GetResimulationErrorPositionThreshold(Chaos::FPhysicsSolverBase::GetResimulationErrorPositionThreshold());
			bShouldTriggerResim = Chaos::FRewindData::CheckVectorThreshold(Target.TargetState.Position, PastState.GetX(), ResimPositionErrorThreshold);
	#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
			if (bShouldTriggerResim)
			{
				DebugColor = FColor::Orange;
			}
	#endif
		}

		// Check for linear velocity discrepancy in Distance / s between client and server
		if (!bShouldTriggerResim && bCompareV)
		{
			const float ResimLinVelocityErrorThreshold = SettingsCurrent.ResimulationSettings.GetResimulationErrorLinearVelocityThreshold(Chaos::FPhysicsSolverBase::GetResimulationErrorLinearVelocityThreshold());
			bShouldTriggerResim = Chaos::FRewindData::CheckVectorThreshold(Target.TargetState.LinVel, PastState.GetV(), ResimLinVelocityErrorThreshold);
	#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
			if (bShouldTriggerResim)
			{
				bResimV = true;
			}
	#endif
		}

		// Check for angular velocity discrepancy in Degrees / s between client and server
		if (!bShouldTriggerResim && bCompareW)
		{
			const float ResimAngVelocityErrorThreshold = SettingsCurrent.ResimulationSettings.GetResimulationErrorAngularVelocityThreshold(Chaos::FPhysicsSolverBase::GetResimulationErrorAngularVelocityThreshold());
			bShouldTriggerResim = Chaos::FRewindData::CheckVectorThreshold(Target.TargetState.AngVel, FMath::RadiansToDegrees(PastState.GetW()), ResimAngVelocityErrorThreshold);
	#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
			if (bShouldTriggerResim)
			{
				bResimW = true;
			}
	#endif
		}

		// Check for rotational discrepancy in Degrees between client and server
		if (!bShouldTriggerResim && bCompareR)
		{
			const float ResimRotationErrorThreshold = SettingsCurrent.ResimulationSettings.GetResimulationErrorRotationThreshold(Chaos::FPhysicsSolverBase::GetResimulationErrorRotationThreshold());
			bShouldTriggerResim = Chaos::FRewindData::CheckQuaternionThreshold(Target.TargetState.Quaternion, PastState.GetR(), ResimRotationErrorThreshold);
	#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
			if (bShouldTriggerResim)
			{
				DebugColor = FColor::Magenta;
			}
	#endif
		}

		// Check for sleeping discrepancy between client and server
		if (!bShouldTriggerResim && bCompareSleep)
		{
			const bool bShouldSleep = (Target.TargetState.Flags & ERigidBodyFlags::Sleeping) != 0;
			const bool bIsSleep = PastState.ObjectState() == Chaos::EObjectStateType::Sleeping;
			bShouldTriggerResim = bIsSleep != bShouldSleep;
	#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
			if (bShouldTriggerResim)
			{
				DebugColor = FColor::White;
			}
	#endif
		}

	#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)

		if (Chaos::FPhysicsSolverBase::CanDebugNetworkPhysicsPrediction())
		{
			UE_LOGF(LogPhysics, Log, "Apply Rigid body state at local frame %d with offset = %d", LocalFrame, NetworkPhysicsTickOffset);
			UE_LOGF(LogPhysics, Log, "Should Trigger Resim = %ls | Server Frame = %d | Client Frame = %d", (bShouldTriggerResim ? TEXT("True") : TEXT("False")), Target.ServerFrame, LocalFrame);
			UE_LOGF(LogPhysics, Log, "Particle Target Position = %ls | Current Position = %ls", *Target.TargetState.Position.ToString(), *PastState.GetX().ToString());
			UE_LOGF(LogPhysics, Log, "Particle Target Velocity = %ls | Current Velocity = %ls", *Target.TargetState.LinVel.ToString(), *PastState.GetV().ToString());
			UE_LOGF(LogPhysics, Log, "Particle Target Quaternion = %ls | Current Quaternion = %ls", *Target.TargetState.Quaternion.ToString(), *PastState.GetR().ToString());
			UE_LOGF(LogPhysics, Log, "Particle Target Omega = %ls | Current Omega= %ls", *Target.TargetState.AngVel.ToString(), *PastState.GetW().ToString());
		}

		if (PhysicsReplicationCVars::ResimulationCVars::bDrawDebug || RenderInterpolationCVars::bRenderInterpDebugDrawResimTrigger)
		{
			if (bShouldTriggerResim)
			{
				FVector Box = RenderInterpolationCVars::bRenderInterpDebugDrawResimTrigger ? FVector(6, 3, 2) : FVector(40, 20, 10);
				Box *= RenderInterpolationCVars::RenderInterpDebugDrawResimBoxScale;
				const float DrawThickness = (RenderInterpolationCVars::bRenderInterpDebugDrawResimTrigger ? 0.5f : 1.5f) * RenderInterpolationCVars::RenderInterpDebugDrawResimBoxScale;

				if (RenderInterpolationCVars::bRenderInterpDebugDrawResimTrigger) // Resim debug draw extension for render interpolation 
				{
					Chaos::FDebugDrawQueue::GetInstance().DrawDebugBox(PastState.GetX(), Box, PastState.GetR(), FColor::White, false, PhysicsReplicationCVars::DebugDrawLifeTime, 0, DrawThickness);
					Chaos::FDebugDrawQueue::GetInstance().DrawDebugBox(Target.TargetState.Position, Box, Target.TargetState.Quaternion, DebugColor, false, PhysicsReplicationCVars::DebugDrawLifeTime, 0, DrawThickness);

					Chaos::FDebugDrawQueue::GetInstance().DrawDebugDirectionalArrow(Handle->GetX(), PastState.GetX(), 5.0f, FColor::White, false, PhysicsReplicationCVars::DebugDrawLifeTime, 0, DrawThickness);
					Chaos::FDebugDrawQueue::GetInstance().DrawDebugDirectionalArrow(PastState.GetX(), Target.TargetState.Position, 5.0f, FColor::Black, false, PhysicsReplicationCVars::DebugDrawLifeTime, 0, DrawThickness);

					if (bResimV)
					{
						const FVector DiffV = Target.TargetState.LinVel - PastState.GetV();
						Chaos::FDebugDrawQueue::GetInstance().DrawDebugDirectionalArrow(Target.TargetState.Position, Target.TargetState.Position + DiffV, 5.0f, FColor::Orange, false, PhysicsReplicationCVars::DebugDrawLifeTime, 0, DrawThickness);
					}
					if (bResimW)
					{
						const FVector DiffW = Target.TargetState.AngVel - FMath::RadiansToDegrees(PastState.GetW());
						Chaos::FDebugDrawQueue::GetInstance().DrawDebugDirectionalArrow(Target.TargetState.Position + DiffW, Target.TargetState.Position, 5.0f, FColor::Magenta, false, PhysicsReplicationCVars::DebugDrawLifeTime, 0, DrawThickness);
					}
				}
				else // Resim trigger debug draw
				{
					Chaos::FDebugDrawQueue::GetInstance().DrawDebugBox(Handle->GetX(), Box, PastState.GetR(), FColor::White, false, PhysicsReplicationCVars::DebugDrawLifeTime, 0, DrawThickness);
					Chaos::FDebugDrawQueue::GetInstance().DrawDebugBox(Handle->GetX() + (Target.TargetState.Position - PastState.GetX()), Box, Target.TargetState.Quaternion, DebugColor, false, PhysicsReplicationCVars::DebugDrawLifeTime, 0, DrawThickness);

					if (bResimV)
					{
						const FVector DiffV = Target.TargetState.LinVel - PastState.GetV();
						Chaos::FDebugDrawQueue::GetInstance().DrawDebugDirectionalArrow(Handle->GetX(), Handle->GetX() + DiffV, 5.0f, FColor::Orange, false, PhysicsReplicationCVars::DebugDrawLifeTime, 0, DrawThickness);
					}
					if (bResimW)
					{
						const FVector DiffW = Target.TargetState.AngVel - FMath::RadiansToDegrees(PastState.GetW());
						Chaos::FDebugDrawQueue::GetInstance().DrawDebugDirectionalArrow(Handle->GetX() + DiffW, Handle->GetX(), 5.0f, FColor::Magenta, false, PhysicsReplicationCVars::DebugDrawLifeTime, 0, DrawThickness);
					}
				}
			}
		}
	#endif
	}

	return bShouldTriggerResim;
}

/** Compare states and trigger resimulation if needed */
bool FPhysicsReplicationAsync::ResimulationReplication(Chaos::FPBDRigidParticleHandle* Handle, FReplicatedPhysicsTargetAsync& Target, const float DeltaSeconds)
{
	Chaos::FPBDRigidsSolver* RigidsSolver = static_cast<Chaos::FPBDRigidsSolver*>(GetSolver());
	if (!RigidsSolver)
	{
		return true;
	}

	Chaos::FRewindData* RewindData = GetOrEnableRewindData(RigidsSolver);

	if (!IsTargetValidForResim(Target) || !Handle)
	{
		return true;
	}

	bool bClearTarget = true;
	if (RewindData)
	{ 
		if(RigidsSolver->GetEvolution())
		{ 
			const int32 LocalFrame = Target.ServerFrame - NetworkPhysicsTickOffset;
			static constexpr Chaos::FFrameAndPhase::EParticleHistoryPhase RewindPhase = Chaos::FFrameAndPhase::EParticleHistoryPhase::PrePushData;

			// Get state from locally cached history for frame corresponding to received data
			const Chaos::FGeometryParticleState PastState = RewindData->GetPastStateAtFrame(*Handle, LocalFrame, RewindPhase);

			const bool bShouldTriggerResim = (Chaos::RewindBeforeAdvance == 0) ? ShouldResimulateParticle(Handle, Target, DeltaSeconds) : false;

			if (bShouldTriggerResim && Target.TickCount == 0 && LocalFrame > RewindData->GetBlockedResimFrame())
			{
				// Request resimulation
				RewindData->RequestResimulation(LocalFrame, Handle);
			}
			else if (SettingsCurrent.ResimulationSettings.GetRuntimeCorrectionEnabled())
			{
				const int32 NumPredictedFrames = RigidsSolver->GetCurrentFrame() - LocalFrame - Target.TickCount;

				if (Target.TickCount <= NumPredictedFrames && NumPredictedFrames > 0)
				{
					const FVector ErrorOffset = (Target.TargetState.Position - PastState.GetX());

					// Positional Correction
					const float CorrectionAmountX = SettingsCurrent.ResimulationSettings.GetPosStabilityMultiplier() / NumPredictedFrames;
					const FVector PosDiffCorrection = ErrorOffset * CorrectionAmountX; // Same result as (ErrorOffset / NumPredictedFrames) * PosStabilityMultiplier
					const FVector CorrectedX = Handle->GetX() + PosDiffCorrection;

					// Rotational Correction
					const float CorrectionAmountR = SettingsCurrent.ResimulationSettings.GetRotStabilityMultiplier() / NumPredictedFrames;
					const FQuat DeltaQuat = PastState.GetR().Inverse() * Target.TargetState.Quaternion;
					const FQuat TargetCorrectionR = Handle->GetR() * DeltaQuat;
					const FQuat CorrectedR = FQuat::Slerp(Handle->GetR(), TargetCorrectionR, CorrectionAmountR);

					if (SettingsCurrent.ResimulationSettings.GetRuntimeVelocityCorrectionEnabled())
					{
						// Linear Velocity Correction
						const FVector LinVelDiff = Target.TargetState.LinVel - PastState.GetV(); // Velocity vector that the server covers but the client doesn't
						const float CorrectionAmountV = SettingsCurrent.ResimulationSettings.GetVelStabilityMultiplier() / NumPredictedFrames;
						const FVector VelCorrection = LinVelDiff * CorrectionAmountV; // Same result as (LinVelDiff / NumPredictedFrames) * VelStabilityMultiplier
						const FVector CorrectedV = Handle->GetV() + VelCorrection;

						// Angular Velocity Correction
						const FVector AngVelDiff = FMath::DegreesToRadians(Target.TargetState.AngVel) - PastState.GetW(); // Angular velocity vector that the server covers but the client doesn't
						const float CorrectionAmountW = SettingsCurrent.ResimulationSettings.GetAngVelStabilityMultiplier() / NumPredictedFrames;
						const FVector AngVelCorrection = AngVelDiff * CorrectionAmountW; // Same result as (AngVelDiff / NumPredictedFrames) * VelStabilityMultiplier
						const FVector CorrectedW = Handle->GetW() + AngVelCorrection;
					
						// Apply correction to velocities
						Handle->SetV(CorrectedV);
						Handle->SetW(CorrectedW);
					}

		#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
					if (PhysicsReplicationCVars::ResimulationCVars::bDrawDebug)
					{
						Chaos::FDebugDrawQueue::GetInstance().DrawDebugDirectionalArrow(Handle->GetX(), CorrectedX, 5.0f, FColor::MakeRandomSeededColor(LocalFrame), false, PhysicsReplicationCVars::DebugDrawLifeTime, 0, 0.5f);
					}
		#endif
					// Apply correction to position and rotation
					RigidsSolver->GetEvolution()->ApplyParticleTransformCorrection(Handle, CorrectedX, CorrectedR, SettingsCurrent.ResimulationSettings.GetRuntimeCorrectConnectedBodies(), /*bInRecalculateFrictionOnConnectedBodies*/true, ReplicatedParticleIDs);
				}

				// Keep target for NumPredictedFrames time to perform runtime corrections with until a new target is received
				bClearTarget = Target.TickCount >= NumPredictedFrames;
			}

			const bool bShouldSleep = (Target.TargetState.Flags & ERigidBodyFlags::Sleeping) != 0;
			if (Target.IsWaiting() && !bShouldSleep)
			{
				// Don't clear the target if we are waiting for a specific target frame and not sleeping
				bClearTarget = false;
			}
		}
	}

	return bClearTarget;
}

void FPhysicsReplicationAsync::DebugDrawReplicationMode(Chaos::FConstPhysicsObjectHandle PhysicsObjectHandle, const FReplicatedPhysicsTargetAsync& Target)
{
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)

	if (!PhysicsReplicationCVars::DebugDrawShowRepMode)
	{
		return;
	}

	if (PhysicsObjectHandle == nullptr)
	{
		return;
	}

	FVector BoxExtent = FVector(10.0f, 10.0f, 10.0f);
	Chaos::FReadPhysicsObjectInterface_Internal Interface = Chaos::FPhysicsObjectInternalInterface::GetRead();
	if (Chaos::FGeometryParticleHandle* Handle = Interface.GetParticle(PhysicsObjectHandle))
	{
		BoxExtent = Handle->LocalBounds().Extents() * 0.5f;
	}

	FColor DebugColor = FColor::White;
	const EPhysicsReplicationMode RepMode = Target.IsWaiting() ? Target.RepModeOverride : Target.RepMode;
	switch (RepMode)
	{
	case EPhysicsReplicationMode::PredictiveInterpolation:
		DebugColor = FColor::Yellow;
		break;
	case EPhysicsReplicationMode::Resimulation:
		DebugColor = FColor::Red;
		break;
	case EPhysicsReplicationMode::Default:
		DebugColor = FColor::Cyan;
		break;
	case EPhysicsReplicationMode::None:
	default:
		DebugColor = FColor::Black;
		break;
	}

	Chaos::FDebugDrawQueue::GetInstance().DrawDebugBox(Target.TargetState.Position, BoxExtent, Target.TargetState.Quaternion, DebugColor, false, PhysicsReplicationCVars::DebugDrawLifeTime, 0, 1.0f);
#endif
}

FName FPhysicsReplicationAsync::GetFNameForStatId() const
{
	const static FLazyName StaticName("FPhysicsReplicationAsyncCallback");
	return StaticName;
}

bool FPhysicsReplicationAsync::UsePhysicsReplicationLOD()
{
	Chaos::FPBDRigidsSolver* RigidsSolver = static_cast<Chaos::FPBDRigidsSolver*>(GetSolver());
	if (RigidsSolver == nullptr)
	{
		return false;
	}

	IPhysicsReplicationLODAsync* PhysRepLod = RigidsSolver->GetPhysicsReplicationLOD_Internal();
	return PhysRepLod && PhysRepLod->IsEnabled();
}

#pragma endregion // FPhysicsReplicationAsync




bool FPhysicsReplication::ShouldSkipPhysicsReplication()
{
	return (CharacterMovementCVars::SkipPhysicsReplication != 0);
}

UWorld* FPhysicsReplication::GetOwningWorld()
{
	return PhysScene ? PhysScene->GetOwningWorld() : nullptr;
}

const UWorld* FPhysicsReplication::GetOwningWorld() const
{
	return PhysScene ? PhysScene->GetOwningWorld() : nullptr;
}

float FPhysicsReplication::GetLocalPing() const
{
	if (const UWorld* World = GetOwningWorld())
	{
		if (APlayerController* PlayerController = World->GetFirstPlayerController())
		{
			if (const TObjectPtr<APlayerState>& PlayerState = PlayerController->PlayerState)
			{
				if (PlayerState.GetResidence() != EResidence::Remote)
				{
					return PlayerState->ExactPing;
				}
			}
		}

	}
	return 0.0f;
}

float FPhysicsReplication::GetOwnerPing(const AActor* const Owner, const FReplicatedPhysicsTarget& Target) const
{
	//
	// NOTE: At the moment, we have no real way to objectively access the ping of the
	// authoritative simulation owner to the server, which is what this function
	// claims to return.
	//
	// In order to actually use ping to extrapolate replication, we need to access
	// it with something along the lines of the disabled code below.
	//
#if false
	if (UPlayer* OwningPlayer = OwningActor->GetNetOwningPlayer())
	{
		if (UWorld* World = GetOwningWorld())
		{
			if (APlayerController* PlayerController = OwningPlayer->GetPlayerController(World))
			{
				if (APlayerState* PlayerState = PlayerController->PlayerState)
				{
					return PlayerState->ExactPing;
				}
			}
		}
	}
#endif

	return 0.0f;
}