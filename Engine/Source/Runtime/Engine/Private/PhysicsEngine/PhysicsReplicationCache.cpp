// Copyright Epic Games, Inc. All Rights Reserved.

#include "Physics/PhysicsReplicationCache.h"
#include "Physics/Experimental/PhysScene_Chaos.h"
#include "PBDRigidsSolver.h"
#include "Chaos/PhysicsObjectInternalInterface.h"
#include "Components/PrimitiveComponent.h"
#include "Physics/NetworkPhysicsComponent.h"

namespace ReplicationCacheCvars
{
	float LingerForSeconds = 1.0f;
	FAutoConsoleVariableRef CVar_ReplicationCacheLingerForSeconds(TEXT("np2.ReplicationCache.LingerForNSeconds"), LingerForSeconds, TEXT("How long to keep data in the replication cache without the actor accessing it, after this we stop caching the actors state until it tries to access it again."));
}



// -------------- Game Thread --------------

FPhysicsReplicationCache::FPhysicsReplicationCache(FPhysScene_Chaos* InPhysicsScene)
{
	using namespace Chaos;
	
	PhysicsScene = InPhysicsScene;
	check(PhysicsScene);

	// Create and register the async flow
	AsyncPhysicsReplicationCache = nullptr;
	if (Chaos::FPhysicsSolver* Solver = PhysicsScene->GetSolver())
	{
		AsyncPhysicsReplicationCache = Solver->CreateAndRegisterSimCallbackObject_External<FPhysicsReplicationCacheAsync>();
	}
}

FPhysicsReplicationCache::~FPhysicsReplicationCache()
{
	// Unregister and free the async flow
	if (AsyncPhysicsReplicationCache && PhysicsScene)
	{
		if (Chaos::FPhysicsSolver* Solver = PhysicsScene->GetSolver())
		{
			Solver->UnregisterAndFreeSimCallbackObject_External(AsyncPhysicsReplicationCache);
		}
	}
}

const FRigidBodyState* FPhysicsReplicationCache::GetStateFromReplicationCache(UPrimitiveComponent* RootComponent, int32& OutSolverFrame)
{
	OutSolverFrame = 0;

	if (!RootComponent)
	{
		return nullptr;
	}

	if (Chaos::FPhysicsObjectHandle PhysicsObject = RootComponent->GetPhysicsObjectByName(NAME_None))
	{
		return GetStateFromReplicationCache(PhysicsObject, OutSolverFrame);
	}

	return nullptr;
}

const FRigidBodyState* FPhysicsReplicationCache::GetStateFromReplicationCache(Chaos::FPhysicsObjectHandle PhysicsObject, int32& OutSolverFrame)
{
	FRigidBodyState* ReplicationState = nullptr;
	OutSolverFrame = 0;

	if (!AsyncPhysicsReplicationCache)
	{
		return ReplicationState;
	}

	// Process async output to get the latest cache update
	ProcessAsyncOutput();

	// Get the state for specified component or register for replication cache
	if (PhysicsObject)
	{
		if (FPhysicsReplicationCacheAsyncInput* AsyncInput = AsyncPhysicsReplicationCache->GetProducerInputData_External())
		{
			// Register physics object in internal replication cache
			AsyncInput->AccessedObjects.Add(PhysicsObject);
		}

		// Get the cached state if there is one, returns nullptr if no state is cached, which mainly happens on the first call since we haven't started caching this particles state yet at that point.
		ReplicationState = ReplicationCache_External.Find(PhysicsObject);
		
		if (ReplicationState)
		{
			// Return the solver frame through the out parameter
			OutSolverFrame = SolverFrame;
		}
	}

	return ReplicationState;
}

/** Process async output to populate the replication cache on the game thread.
* Called from FPhysicsReplicationCacheAsync::ProcessOutputs_External() which gets called as soon as a new output is available */
void FPhysicsReplicationCache::ProcessAsyncOutput()
{
	if (!AsyncPhysicsReplicationCache)
	{
		return;
	}

	// Receive state from Physics Thread
	while (Chaos::TSimCallbackOutputHandle<FPhysicsReplicationCacheAsyncOutput> AsyncOutput = AsyncPhysicsReplicationCache->PopFutureOutputData_External())
	{
		// We only care about the latest data from the physics thread
		if (!AsyncPhysicsReplicationCache->IsOutputQueueEmpty_External())
		{
			continue;
		}

		if (AsyncOutput->SolverFrame <= SolverFrame)
		{
			continue;
		}

		// The key array and value array should always be populated with the same count
		check(AsyncOutput->ReplicationCache_Key_Marshal.Num() == AsyncOutput->ReplicationCache_Value_Marshal.Num());

		const int32 CacheSize = AsyncOutput->ReplicationCache_Key_Marshal.Num();

		SolverFrame = AsyncOutput->SolverFrame;
		ReplicationCache_External.Empty(CacheSize);

		for (int32 Idx = 0; Idx < CacheSize; Idx++)
		{
			Chaos::FConstPhysicsObjectHandle PhysicsObject = AsyncOutput->ReplicationCache_Key_Marshal[Idx];
			ReplicationCache_External.Emplace(PhysicsObject, AsyncOutput->ReplicationCache_Value_Marshal[Idx]);
		}
	}
}

void FPhysicsReplicationCache::RegisterForReplicationCache(UPrimitiveComponent* RootComponent)
{
	if (!RootComponent || !AsyncPhysicsReplicationCache)
	{
		return;
	}

	if (Chaos::FPhysicsObjectHandle PhysicsObject = RootComponent->GetPhysicsObjectByName(NAME_None))
	{
		if (FPhysicsReplicationCacheAsyncInput* AsyncInput = AsyncPhysicsReplicationCache->GetProducerInputData_External())
		{
			// Register physics object in internal replication cache
			AsyncInput->AccessedObjects.Add(PhysicsObject);
		}
	}
}

void FPhysicsReplicationCache::UnregisterForReplicationCache(UPrimitiveComponent* RootComponent)
{
	if (!RootComponent || !AsyncPhysicsReplicationCache)
	{
		return;
	}

	if (Chaos::FPhysicsObjectHandle PhysicsObject = RootComponent->GetPhysicsObjectByName(NAME_None))
	{
		// Clear physics object from external replication cache
		ReplicationCache_External.Remove(PhysicsObject);

		// Clear physics object from internal replication cache
		if (FPhysicsReplicationCacheAsyncInput* AsyncInput = AsyncPhysicsReplicationCache->GetProducerInputData_External())
		{
			AsyncInput->UnregisterObjects.Add(PhysicsObject);
		}
	}
}



// -------------- Physics Thread --------------

void FPhysicsReplicationCacheAsync::OnPreSimulate_Internal()
{
	ProcessAsyncInputs();
}

void FPhysicsReplicationCacheAsync::OnPostSolve_Internal()
{
	PopulateReplicationCache_Internal();
}

void FPhysicsReplicationCacheAsync::OnPhysicsObjectUnregistered_Internal(Chaos::FConstPhysicsObjectHandle PhysicsObject)
{
	// Unregister physics object from caching state
	ReplicationCache_Internal.Remove(PhysicsObject);
	if (const FPhysicsReplicationCacheAsyncInput* AsyncInput = GetConsumerInput_Internal())
	{
		// Remove from AsyncInput->AccessedObjects so that we don't potentially add this back again in ProcessAsyncInputs()
		AsyncInput->AccessedObjects.Remove(PhysicsObject);
	}
	bUpdateAfterRemoval = true; // Ensure we produce an async output after this removal, even if the cache is empty
}

void FPhysicsReplicationCacheAsync::ProcessAsyncInputs()
{
	// Process async inputs from the game thread
	if (const FPhysicsReplicationCacheAsyncInput* AsyncInput = GetConsumerInput_Internal())
	{
		if (Chaos::FPBDRigidsSolver* RigidSolver = static_cast<Chaos::FPBDRigidsSolver*>(GetSolver()))
		{
			for (const Chaos::FConstPhysicsObjectHandle& PhysicsObject : AsyncInput->AccessedObjects)
			{
				// Register physics object to cache state
				FPhysicsReplicationCacheData& CacheData = ReplicationCache_Internal.FindOrAdd(PhysicsObject);

				// Register time this data was accessed, used to stop caching data if the object stops getting accessed
				CacheData.SetAccessTime(RigidSolver->GetSolverTime());
			}
		}

		for (const Chaos::FConstPhysicsObjectHandle& PhysicsObject : AsyncInput->UnregisterObjects)
		{
			// Unregister physics object from caching state
			ReplicationCache_Internal.Remove(PhysicsObject);
			bUpdateAfterRemoval = true; // Ensure we produce an async output after this removal, even if the cache is empty
		}
	}
}

void FPhysicsReplicationCacheAsync::PopulateReplicationCache_Internal()
{
	// Early out if the cache is empty, unless we have recently removed an entry from the cache, then we need to send one last update even if the cache is empty to clear the game thread cache
	if (ReplicationCache_Internal.Num() == 0 && !bUpdateAfterRemoval)
	{
		return;
	}

	Chaos::FPBDRigidsSolver* RigidSolver = static_cast<Chaos::FPBDRigidsSolver*>(GetSolver());
	if (!RigidSolver)
	{
		return;
	}

	FPhysicsReplicationCacheAsyncOutput& AsyncOutput = GetProducerOutputData_Internal();
	
	// +1 Because we cache this in PostSolve which has the end result of the physics solve which is equal to the starting state of next physics frame which is what this state correspond to
	AsyncOutput.SolverFrame = RigidSolver->GetCurrentFrame() + 1;

	// Setup the arrays to marshal replication cache with correct allocation size
	AsyncOutput.ReplicationCache_Key_Marshal.SetNum(ReplicationCache_Internal.Num());
	AsyncOutput.ReplicationCache_Value_Marshal.SetNum(ReplicationCache_Internal.Num());
	int32 MarshalIndex = 0;

	// Iterate over all physics objects in the cache and marshal their state back to the game thread
	for (auto It = ReplicationCache_Internal.CreateIterator(); It; ++It)
	{
		FPhysicsReplicationCacheData& ReplicationCacheData = It.Value();

		// Check if the object has lingered too long without being accessed
		if (ReplicationCacheData.GetAccessTime() < (RigidSolver->GetSolverTime() - ReplicationCacheCvars::LingerForSeconds))
		{
			// Unregister physics object from caching state
			PendingDeleteFromReplicationCache_Internal.Add(It.Key());

			// Set empty entry in output since it's already allocated
			AsyncOutput.ReplicationCache_Key_Marshal[MarshalIndex] = nullptr;
			MarshalIndex++;
		
			continue;
		}

		// Get current state and marshal to game thread
		const Chaos::FConstPhysicsObjectHandle& PhysicsObject = It.Key();
		if (PhysicsObject)
		{
			Chaos::FReadPhysicsObjectInterface_Internal Interface = Chaos::FPhysicsObjectInternalInterface::GetRead();
			if (Chaos::FPBDRigidParticleHandle* Handle = Interface.GetRigidParticle(PhysicsObject))
			{
				FRigidBodyState& ReplicationState = ReplicationCacheData.GetState();
				ReplicationState.Position = Handle->GetP();
				ReplicationState.Quaternion = Handle->GetQ();
				ReplicationState.LinVel = Handle->GetV();
				ReplicationState.AngVel = FMath::RadiansToDegrees(Handle->GetW());
				ReplicationState.Flags = Handle->ObjectState() == Chaos::EObjectStateType::Sleeping ? ERigidBodyFlags::Sleeping : 0;

				// Marshal state
				AsyncOutput.ReplicationCache_Key_Marshal[MarshalIndex] = PhysicsObject;
				AsyncOutput.ReplicationCache_Value_Marshal[MarshalIndex] = ReplicationState;
				MarshalIndex++;
			}
		}
	}

	for (Chaos::FConstPhysicsObjectHandle& PhysicsObject : PendingDeleteFromReplicationCache_Internal)
	{
		ReplicationCache_Internal.Remove(PhysicsObject);
	}
	PendingDeleteFromReplicationCache_Internal.Reset();

	bUpdateAfterRemoval = false;

	// Mark async output as final, meaning it can be read on the game thread instantly instead of at the end of the physics tick
	FinalizeOutputData_Internal();
}