// Copyright Epic Games, Inc. All Rights Reserved.

#include "PhysicsReplicationLOD.h"
#include "Physics/Experimental/PhysScene_Chaos.h"
#include "PBDRigidsSolver.h"
#include "Chaos/PhysicsObjectInternalInterface.h"
#include <Components/PrimitiveComponent.h>
#include "Chaos/ParticleDirtyFlags.h"
#include "Chaos/DebugDrawQueue.h"
#include "Chaos/Island/IslandManager.h"


namespace PhysicsReplicationLODCVars
{
	int32 OverrideEnabled = 0;
	static FAutoConsoleVariableRef CVarOverrideEnabled(TEXT("p.ReplicationLOD.OverrideEnabled"), OverrideEnabled, TEXT("0 = Use Project Settings or API calls. 1 = Override to enabled. -1 = Override to disabled."));

	int32 TransitionExtrapFrameMin = 3;
	static FAutoConsoleVariableRef CVarTransitionExtrapFrameMin(TEXT("p.ReplicationLOD.TransitionExtrapFrameMin"), TransitionExtrapFrameMin, TEXT("Minimum number of frames closest to the latest received server state that always use Extrapolation (PI + extrapolated target) inside the transition zone."));

	float TransitionExtrapFraction = 0.3f;
	static FAutoConsoleVariableRef CVarTransitionExtrapFraction(TEXT("p.ReplicationLOD.TransitionExtrapFraction"), TransitionExtrapFraction, TEXT("Fraction (0.0 to 1.0) of per-particle forward-prediction frames handled by Extrapolation (PI + extrapolated target) inside the transition zone. Beyond this fraction, particles use ResimDecay (Resimulation + SimulationDecay). Set to 1.0 for pure Extrapolation or 0.0 with TransitionExtrapFrameMin=0 for pure ResimDecay."));

	bool bTransitionModeDebugLog = false;
	static FAutoConsoleVariableRef CVarTransitionModeDebugLog(TEXT("p.ReplicationLOD.TransitionMode.DebugLog"), bTransitionModeDebugLog, TEXT("When enabled, log per-particle LOD-transition decisions each regular frame and SimulationDecay during resim."));

	float MinimumBaseDistance = -1.0f;
	static FAutoConsoleVariableRef CVarMinimumBaseDistance(TEXT("p.ReplicationLOD.MinimumBaseDistance"), MinimumBaseDistance, TEXT("Override the minimum distance in the physics replication LODs Base Distance. Negative value disables the override and project settings will apply."));

	float BaseDistanceRadiusMultiplier = -1.0f;
	static FAutoConsoleVariableRef CVarBaseDistanceRadiusMultiplier(TEXT("p.ReplicationLOD.BaseDistanceRadiusMultiplier"), BaseDistanceRadiusMultiplier, TEXT("Override the distance radius multiplier which adds focal particles bounding box radius to the physics replication LODs Base Distance. Value of 0.75f adds 75% of the radius on top of the LOD distances. Negative value disables the override and project settings will apply."));

	float BaseDistancesForResimulationMode = -1.0f;
	static FAutoConsoleVariableRef CVarBaseDistancesForResimulationMode(TEXT("p.ReplicationLOD.BaseDistancesForResimulationMode"), BaseDistancesForResimulationMode, TEXT("Override the amount of Base Distance to set the distance for where within resimulation will be recommended as the replication mode by LOD. Negative value disables the override and project settings will apply."));

	float BaseDistancesForFullPrediction = -1.0f;
	static FAutoConsoleVariableRef CVarBaseDistancesForFullPrediction(TEXT("p.ReplicationLOD.BaseDistancesForFullPrediction"), BaseDistancesForFullPrediction, TEXT("Override the amount of Base Distance to set the distance where within full forward prediction to clients current timeline will be recommended. Negative value disables the override and project settings will apply."));

	float TimeOverDistance = -1.0f;
	static FAutoConsoleVariableRef CVaTimeOverDistance(TEXT("p.ReplicationLOD.TimeOverDistance"), TimeOverDistance, TEXT("Time/Distance LOD alignment value in milliseconds/centimeter. Used as Distance * TimeOverDistance = Time, where Time is how far behind the current timeline replication should run. Start outside of DistanceFullPrediction and clamped by the received states timeline so we don't extrapolate backwards. Negative value disables the override and project settings will apply."));

	bool bDrawDebugEnabled = false;
	static FAutoConsoleVariableRef CVarDrawDebug(TEXT("p.ReplicationLOD.DrawDebug.Enabled"), bDrawDebugEnabled, TEXT(""));

	float DrawDebugWorstLatency = 300.0f;
	static FAutoConsoleVariableRef CVarDrawDebugWorstLatency(TEXT("p.ReplicationLOD.DrawDebug.WorstLatency"), DrawDebugWorstLatency, TEXT("Used to calculate color gradient in debug draw from forward predicted to worst expected latency."));
}

// ----------- Game Thread API -----------

FPhysicsReplicationLOD::FPhysicsReplicationLOD(FPhysScene* InPhysicsScene)
	: bEnabled(false)
	, PhysScene(InPhysicsScene)
	, PhysicsReplicationLODAsync(nullptr)
{
	// Apply project settings
	FPhysicsReplicationLODSettings& LODSettings = UPhysicsSettings::Get()->PhysicsPrediction.PhysicsReplicationLODSettings;
	bEnabled = LODSettings.bEnablePhysicsReplicationLOD;

	if (Chaos::FPhysicsSolverBase* Solver = PhysScene->GetSolver())
	{
		// Create physics thread instance and cache project settings there
		PhysicsReplicationLODAsync = Solver->CreateAndRegisterSimCallbackObject_External<FPhysicsReplicationLODAsync>();
		PhysicsReplicationLODAsync->bEnabled = LODSettings.bEnablePhysicsReplicationLOD;
		PhysicsReplicationLODAsync->DefaultSettings = LODSettings;
	}
}

FPhysicsReplicationLOD::~FPhysicsReplicationLOD()
{
	if (PhysicsReplicationLODAsync)
	{
		if (Chaos::FPhysicsSolverBase* Solver = PhysScene->GetSolver())
		{
			Solver->UnregisterAndFreeSimCallbackObject_External(PhysicsReplicationLODAsync);
		}
	}
}

void FPhysicsReplicationLOD::SetEnabled(const bool bInEnabled)
{
	bEnabled = bInEnabled;
	
	// Marshal over the enabled setting to the physics thread
	if (PhysicsReplicationLODAsync)
	{
		if (FPhysicsReplicationLODAsyncInput* AsyncInput = PhysicsReplicationLODAsync->GetProducerInputData_External())
		{
			AsyncInput->bEnabled = bEnabled;
		}
	}
}

bool FPhysicsReplicationLOD::IsEnabled() const
{
	if (PhysicsReplicationLODCVars::OverrideEnabled == -1)
	{
		return false;
	}
	return bEnabled || (PhysicsReplicationLODCVars::OverrideEnabled == 1);
}

void FPhysicsReplicationLOD::RegisterFocalPoint_External(const UPrimitiveComponent* Component, FName BoneName)
{
	if (Component)
	{
		RegisterFocalPoint_External(Component->GetPhysicsObjectByName(BoneName));
	}
}

void FPhysicsReplicationLOD::UnregisterFocalPoint_External(const UPrimitiveComponent* Component, FName BoneName)
{
	if (Component)
	{
		UnregisterFocalPoint_External(Component->GetPhysicsObjectByName(BoneName));
	}
}

void FPhysicsReplicationLOD::RegisterFocalPoint_External(Chaos::FConstPhysicsObjectHandle PhysicsObject)
{
	if (PhysicsObject && PhysicsReplicationLODAsync)
	{
		// Marshal registration over to physics thread
		if (FPhysicsReplicationLODAsyncInput* AsyncInput = PhysicsReplicationLODAsync->GetProducerInputData_External())
		{
			AsyncInput->PhysicsObjectsToRegister.Add(PhysicsObject);
		}
	}
}

void FPhysicsReplicationLOD::UnregisterFocalPoint_External(Chaos::FConstPhysicsObjectHandle PhysicsObject)
{
	if (PhysicsObject && PhysicsReplicationLODAsync)
	{
		// Marshal deregistration over to physics thread
		if (FPhysicsReplicationLODAsyncInput* AsyncInput = PhysicsReplicationLODAsync->GetProducerInputData_External())
		{
			AsyncInput->PhysicsObjectsToUnregister.Add(PhysicsObject);
		}
	}
}

IPhysicsReplicationLODAsync* FPhysicsReplicationLOD::GetPhysicsReplicationLOD_Internal()
{
	return PhysicsReplicationLODAsync;
}



// ----------- Physics Thread API -----------

bool FPhysicsReplicationLODAsync::IsEnabled() const
{
	if (PhysicsReplicationLODCVars::OverrideEnabled == -1)
	{
		return false;
	}
	return (bEnabled || (PhysicsReplicationLODCVars::OverrideEnabled == 1));
}

void FPhysicsReplicationLODAsync::OnPostInitialize_Internal()
{
	Chaos::FPBDRigidsSolver* RigidsSolver = static_cast<Chaos::FPBDRigidsSolver*>(GetSolver());
	if (RigidsSolver == nullptr)
	{
		return;
	}

	// Register this physics replication LOD in the solver
	RigidsSolver->SetPhysicsReplicationLOD_Internal(this);
}

// ProcessInputs_Internal gets called before OnPreSimulate_Internal
void FPhysicsReplicationLODAsync::ProcessInputs_Internal(int32 PhysicsStep)
{
	Chaos::FPBDRigidsSolver* RigidsSolver = static_cast<Chaos::FPBDRigidsSolver*>(GetSolver());
	if (RigidsSolver == nullptr)
	{
		return;
	}

	// Process incoming marshaled data from game thread; deduped against an early GetLODData_Internal call that already consumed for this step.
	const int32 CurrentStep = RigidsSolver->GetCurrentFrame();
	if (LastInputProcessStep != CurrentStep)
	{
		ConsumeAsyncInput();
		LastInputProcessStep = CurrentStep;
	}
}

void FPhysicsReplicationLODAsync::OnPostSolve_Internal()
{
	// Clear the cached LOD data for interacting particles without reallocating
	CachedIslandLodData.Reset();
}

void FPhysicsReplicationLODAsync::CacheParticlesInFocalIslands()
{
	Chaos::FPBDRigidsSolver* RigidsSolver = static_cast<Chaos::FPBDRigidsSolver*>(GetSolver());
	if (RigidsSolver == nullptr)
	{
		return;
	}

	ParticlesInFocalIslands.Reset();

	Chaos::Private::FPBDIslandManager& IslandManager = RigidsSolver->GetEvolution()->GetIslandManager();
	Chaos::FReadPhysicsObjectInterface_Internal Interface = Chaos::FPhysicsObjectInternalInterface::GetRead();

	const int32 FocalParticlesIndexMax = FocalParticles.Num() - 1;
	for (int32 i = FocalParticlesIndexMax; i >= 0; i--)
	{
		Chaos::FConstPhysicsObjectHandle FocalParticle = FocalParticles[i];
		Chaos::FGeometryParticleHandle* FocalParticleHandle = Interface.GetParticle(FocalParticle);
		
		if (!FocalParticleHandle)
		{
			// If focal particle is no longer valid, remove it
			FocalParticles.RemoveAt(i);
			continue;
		}

		// Get a list of particles from the same island as a focal particle is in, i.e. particles interacting with a resim particle
		ParticleIslands.Reset();
		IslandParticles.Reset();
		IslandManager.FindParticleIslands(FocalParticleHandle, OUT ParticleIslands);
		IslandManager.FindParticlesInIslands(ParticleIslands, OUT IslandParticles);
		for (const Chaos::FGeometryParticleHandle* IslandParticle : IslandParticles)
		{
			if (IslandParticle->GetParticleType() != Chaos::EParticleType::Static)
			{
				// Add all particles that are in the same island as a focal particle to an array
				ParticlesInFocalIslands.Add(IslandParticle->GetHandleIdx());
			}
		}
	}
}

void FPhysicsReplicationLODAsync::ConsumeAsyncInput()
{
	if (const FPhysicsReplicationLODAsyncInput* AsyncInput = GetConsumerInput_Internal())
	{
		if (AsyncInput->bEnabled.IsSet())
		{
			bEnabled = *AsyncInput->bEnabled;
		}

		Chaos::FReadPhysicsObjectInterface_Internal Interface = Chaos::FPhysicsObjectInternalInterface::GetRead();
		for (Chaos::FConstPhysicsObjectHandle PhysicsObject : AsyncInput->PhysicsObjectsToRegister)
		{
			RegisterFocalPoint_Internal(PhysicsObject);
		}

		for (Chaos::FConstPhysicsObjectHandle PhysicsObject : AsyncInput->PhysicsObjectsToUnregister)
		{
			UnregisterFocalPoint_Internal(PhysicsObject);
		}
	}
}

void FPhysicsReplicationLODAsync::OnPreSimulate_Internal()
{
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	if (PhysicsReplicationLODCVars::bDrawDebugEnabled)
	{
		Chaos::FReadPhysicsObjectInterface_Internal Interface = Chaos::FPhysicsObjectInternalInterface::GetRead();
		for (Chaos::FConstPhysicsObjectHandle FocalParticle : FocalParticles)
		{
			if (Chaos::FGeometryParticleHandle* FocalParticleHandle = Interface.GetParticle(FocalParticle))
			{
				// LOD Settings
				const float MinimumBaseDistance = PhysicsReplicationLODCVars::MinimumBaseDistance >= 0.0f ? PhysicsReplicationLODCVars::MinimumBaseDistance : DefaultSettings.MinimumBaseDistance;
				const float BaseDistanceRadiusMultiplier = PhysicsReplicationLODCVars::BaseDistanceRadiusMultiplier >= 0.0f ? PhysicsReplicationLODCVars::BaseDistanceRadiusMultiplier : DefaultSettings.BaseDistanceRadiusMultiplier;
				const float BaseDistancesForFullPrediction = PhysicsReplicationLODCVars::BaseDistancesForFullPrediction >= 0.0f ? PhysicsReplicationLODCVars::BaseDistancesForFullPrediction : DefaultSettings.BaseDistancesForFullPrediction;
				const float BaseDistancesForResimulationMode = PhysicsReplicationLODCVars::BaseDistancesForResimulationMode >= 0.0f ? PhysicsReplicationLODCVars::BaseDistancesForResimulationMode : DefaultSettings.BaseDistancesForResimulationMode;

				// Focal particle specific LOD settings
				const float BaseDistance = MinimumBaseDistance + (FocalParticleHandle->LocalBounds().CenterRadius() * BaseDistanceRadiusMultiplier);
				const float DistanceForFullPrediction = BaseDistance * BaseDistancesForFullPrediction;
				const float DistanceForResimulationMode = BaseDistance * BaseDistancesForResimulationMode;

				const FColor DebugColorFullPrediction = IsEnabled() ? FColor::Blue : FColor::Black;
				const FColor DebugColorResim = IsEnabled() ? FColor::Red : FColor::Black;

				Chaos::FDebugDrawQueue::GetInstance().DrawDebugSphere(FocalParticleHandle->WorldSpaceInflatedBounds().Center(), FocalParticleHandle->LocalBounds().CenterRadius() + DistanceForFullPrediction, 8, DebugColorFullPrediction, false, -1.0f, 0, 1.0f);
				Chaos::FDebugDrawQueue::GetInstance().DrawDebugSphere(FocalParticleHandle->WorldSpaceInflatedBounds().Center(), FocalParticleHandle->LocalBounds().CenterRadius() + DistanceForResimulationMode, 8, DebugColorResim, false, -1.0f, 0, 1.0f);
			}
		}
	}
#endif
}

void FPhysicsReplicationLODAsync::OnPhysicsObjectUnregistered_Internal(Chaos::FConstPhysicsObjectHandle PhysicsObject)
{
	UnregisterFocalPoint_Internal(PhysicsObject);
}

void FPhysicsReplicationLODAsync::RegisterFocalPoint_Internal(Chaos::FConstPhysicsObjectHandle PhysicsObject)
{
	if (!PhysicsObject)
	{
		return;
	}

	FocalParticles.AddUnique(PhysicsObject);
}

void FPhysicsReplicationLODAsync::UnregisterFocalPoint_Internal(Chaos::FConstPhysicsObjectHandle PhysicsObject)
{
	FocalParticles.Remove(PhysicsObject);
}

FPhysicsRepLodData* FPhysicsReplicationLODAsync::GetLODData_Internal(Chaos::FConstPhysicsObjectHandle PhysicsObject, const uint32 LODFlags)
{
	Chaos::FPBDRigidsSolver* RigidsSolver = static_cast<Chaos::FPBDRigidsSolver*>(GetSolver());
	if (RigidsSolver == nullptr)
	{
		return nullptr;
	}

	const int32 CurrentStep = RigidsSolver->GetCurrentFrame();

	// Pull marshaled input on demand if a consumer queries LOD before ProcessInputs_Internal runs this step
	if (LastInputProcessStep != CurrentStep)
	{
		ConsumeAsyncInput();
		LastInputProcessStep = CurrentStep;
	}

	if (!IsEnabled())
	{
		return nullptr;
	}

	if (FocalParticles.Num() == 0)
	{
		// No focal particles registered, return DataAssigned + AlignedFrame=0 (default) which is only a valid return when there are no focal particles
		LodData.Reset();
		LodData.DataAssigned = true;
		return &LodData;
	}

	Chaos::FReadPhysicsObjectInterface_Internal Interface = Chaos::FPhysicsObjectInternalInterface::GetRead();
	const Chaos::FGeometryParticleHandle* ParticleHandle = Interface.GetParticle(PhysicsObject);
	if (!ParticleHandle)
	{
		return nullptr;
	}

	// If PhysicsObject is a focal particle itself, early out
	if (FocalParticles.Contains(PhysicsObject))
	{
		return nullptr;
	}

	// Rebuild the focal-island cache lazily on the first LOD query of the step (island indices are invalidated each step)
	if (LastFocalCacheStep != CurrentStep)
	{
		CacheParticlesInFocalIslands();
		LastFocalCacheStep = CurrentStep;
	}

	LodData.Reset();

	if (PerformIslandLOD(*ParticleHandle, LODFlags))
	{
		return &LodData;
	}

	if (PerformDistanceLOD(*ParticleHandle, LODFlags))
	{
		return &LodData;
	}

	return nullptr;
}

bool FPhysicsReplicationLODAsync::PerformIslandLOD(const Chaos::FGeometryParticleHandle& ParticleHandle, const uint32 LODFlags)
{
	if ((LODFlags & EPhysicsReplicationLODFlags::LODFlag_IslandCheck) == 0)
	{
		return false;
	}

	// Check if particle is in the same island as a focal particle
	if (ParticlesInFocalIslands.Contains(ParticleHandle.GetHandleIdx()))
	{
		Chaos::FPBDRigidsSolver* RigidsSolver = static_cast<Chaos::FPBDRigidsSolver*>(GetSolver());

		LodData.DataAssigned = true;
		LodData.AlignedFrame = RigidsSolver ? RigidsSolver->GetCurrentFrame() : 0;
		LodData.AlignedTime = 0.0f;
	}
	return LodData.DataAssigned;
}

bool FPhysicsReplicationLODAsync::PerformDistanceLOD(const Chaos::FGeometryParticleHandle& ParticleHandle, const uint32 LODFlags)
{
	if ((LODFlags & EPhysicsReplicationLODFlags::LODFlag_DistanceCheck) == 0)
	{
		return false;
	}

	Chaos::FPBDRigidsSolver* RigidsSolver = static_cast<Chaos::FPBDRigidsSolver*>(GetSolver());
	if (RigidsSolver == nullptr)
	{
		return false;
	}
	Chaos::Private::FPBDIslandManager& IslandManager = RigidsSolver->GetEvolution()->GetIslandManager();

	// Check if island already has cached LOD data, if so, early out and return that.
	ParticleIslands.Reset();
	IslandManager.FindParticleIslands(&ParticleHandle, OUT ParticleIslands);
	for (const Chaos::Private::FPBDIsland* Island : ParticleIslands)
	{
		if (FPhysicsRepLodData* CachedLodData = CachedIslandLodData.Find(Island->GetArrayIndex()))
		{
			// Early Out
			LodData = *CachedLodData;
			return LodData.DataAssigned;
		}
	}

	// Helper function to grow AABB for with all interacting particles
	auto IslandAABBHelper = [this](const Chaos::FGeometryParticleHandle* Particle)
		{
			/* NOTE: Not using Particle->WorldSpaceInflatedBounds() due to it being inflated and also variable in size between physics frames (depending on velocity and rotation?) even for uniform shapes.
			 * Non-stable or oscillating bounds runs the risk of making replication less smooth due to the inconsistencies in the resulting LOD data. */

			ParticleAABB = Particle->LocalBounds();
			ParticleAABB.MoveByVector(Particle->GetX()); // TODO: Might need to use Particle->WorldSpaceInflatedBounds().Center() since X might not be the center of the LocalBounds
			IslandAABB.GrowToInclude(ParticleAABB);
		};

	// Populate a bounding box of interacting particles
	IslandAABB.Clear();
	IslandAABBHelper(&ParticleHandle); // We do this for the particle asking for LOD Data outside of the for-loop because there are scenarios where the particle is not part of the IslandParticles array (if it's kinematic and not interacting with anything)
	IslandParticles.Reset();
	IslandManager.FindParticlesInIslands(ParticleIslands, OUT IslandParticles);
	for (const Chaos::FGeometryParticleHandle* IslandParticle : IslandParticles)
	{
		if (IslandParticle->GetParticleType() == Chaos::EParticleType::Static)
		{
			continue;
		}

		if (IslandParticle->GetHandleIdx() == ParticleHandle.GetHandleIdx())
		{
			continue;
		}

		if (IslandParticle->GetParticleType() == Chaos::EParticleType::Kinematic || IslandParticle->ObjectState() == Chaos::EObjectStateType::Kinematic)
		{
			// If there are movable kinematic particles in the calling particle's island(s), also cache other islands that the kinematic particle is part of so that LOD data gets cached for all relevant islands
			IslandManager.FindParticleIslands(IslandParticle, OUT ParticleIslands);
		}

		IslandAABBHelper(IslandParticle);
	}

	// Get the islands sphere radius and center point
	const float IslandRadius = IslandAABB.CenterRadius();
	const Chaos::FVec3 IslandCenter = IslandAABB.Center();

	Chaos::FReadPhysicsObjectInterface_Internal Interface = Chaos::FPhysicsObjectInternalInterface::GetRead();

	// LOD Settings
	const float MinimumBaseDistance = PhysicsReplicationLODCVars::MinimumBaseDistance >= 0.0f ? PhysicsReplicationLODCVars::MinimumBaseDistance : DefaultSettings.MinimumBaseDistance;
	const float BaseDistanceRadiusMultiplier = PhysicsReplicationLODCVars::BaseDistanceRadiusMultiplier >= 0.0f ? PhysicsReplicationLODCVars::BaseDistanceRadiusMultiplier : DefaultSettings.BaseDistanceRadiusMultiplier;
	const float BaseDistancesForFullPrediction = PhysicsReplicationLODCVars::BaseDistancesForFullPrediction >= 0.0f ? PhysicsReplicationLODCVars::BaseDistancesForFullPrediction : DefaultSettings.BaseDistancesForFullPrediction;
	const float BaseDistancesForResimulationMode = PhysicsReplicationLODCVars::BaseDistancesForResimulationMode >= 0.0f ? PhysicsReplicationLODCVars::BaseDistancesForResimulationMode : DefaultSettings.BaseDistancesForResimulationMode;
	const float TimeOverDistance = PhysicsReplicationLODCVars::TimeOverDistance > 0.0f ? PhysicsReplicationLODCVars::TimeOverDistance : DefaultSettings.TimeOverDistance;

	for (Chaos::FConstPhysicsObjectHandle FocalParticle : FocalParticles)
	{
		Chaos::FGeometryParticleHandle* FocalParticleHandle = Interface.GetParticle(FocalParticle);
		if (!FocalParticleHandle)
		{
			continue;
		}

		const float FocalParticleRadius = FocalParticleHandle->LocalBounds().CenterRadius();
		const Chaos::FVec3 FocalParticleCenter = FocalParticleHandle->WorldSpaceInflatedBounds().Center();

		// Focal particle specific LOD settings
		const float BaseDistance = MinimumBaseDistance + (FocalParticleRadius * BaseDistanceRadiusMultiplier);
		const float DistanceForFullPrediction = BaseDistance * BaseDistancesForFullPrediction;
		const float DistanceForResimulationMode = BaseDistance * BaseDistancesForResimulationMode;

		// Check distance towards focal particles taking the radius of both AABBs into account
		const Chaos::FVec3 PosOffset = FocalParticleCenter - IslandCenter;
		const float Distance = FMath::Max(PosOffset.Size() - FocalParticleRadius - IslandRadius, 0.0f);

		// Calculate recommended time and tick alignment from the current timeline
		float TimeAlignment = FMath::Max((Distance - DistanceForFullPrediction), 0.0f) * TimeOverDistance;
		TimeAlignment = (TimeAlignment * 0.001f); // Convert from ms to s
		const float TickAlignment = TimeAlignment / RigidsSolver->GetAsyncDeltaTime();
		const int32 TickAlignmentDiscrete = FMath::Floor(TickAlignment);

		// Update LodData and keep the highest LOD data
		LodData.AlignedTime = FMath::Max(LodData.AlignedTime, TimeAlignment);
		LodData.AlignedFrame = FMath::Max(LodData.AlignedFrame, RigidsSolver->GetCurrentFrame() - TickAlignmentDiscrete);
		LodData.DataAssigned = true;

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
		if (PhysicsReplicationLODCVars::bDrawDebugEnabled)
		{
			PhysicsReplicationLODCVars::DrawDebugWorstLatency = FMath::Max(PhysicsReplicationLODCVars::DrawDebugWorstLatency, 1.0f); // Clamp to above 1ms, to not divide by 0 or negative value
			float LodAlpha = (TimeAlignment * 1000.0f) / PhysicsReplicationLODCVars::DrawDebugWorstLatency; // Temporary solution for debug draw to use NumPredictedFrames
			LodAlpha = FMath::Min(LodAlpha, 1.0f); // Clamp to 1 or lower

			FColor DebugColor = FLinearColor::MakeFromHSV8((uint8)FMath::CeilToInt32((255.0f * 0.8f) * LodAlpha), 150, 255).ToFColor(false);
			DebugColor.A = 255;
			Chaos::FDebugDrawQueue::GetInstance().DrawDebugDirectionalArrow(IslandCenter, FocalParticleCenter, 1.0f, DebugColor, false, -1.0f, 0, 1.0f);
			Chaos::FDebugDrawQueue::GetInstance().DrawDebugSphere(IslandCenter, IslandRadius, 8, DebugColor, false, -1.0f, 0, 1.0f);
		}
#endif

		if (Distance <= DistanceForResimulationMode)
		{
			break; // Highest LOD reached, don't check further focal particles
		}
	}

	// Cache the LOD data for each island that should use the same LOD
	for (const Chaos::Private::FPBDIsland* Island : ParticleIslands)
	{
		CachedIslandLodData.Add(Island->GetArrayIndex(), LodData);
	}

	return LodData.DataAssigned;
}
