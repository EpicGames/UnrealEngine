// Copyright Epic Games, Inc. All Rights Reserved.

#include "Physics/PhysicsReplicationQuantization.h"

#include "Chaos/PhysicsObjectInternalInterface.h"
#include "Chaos/ParticleHandle.h"
#include "Components/PrimitiveComponent.h"
#include "Engine/World.h"
#include "GameFramework/Actor.h"
#include "GameFramework/PlayerController.h"
#include "Net/Core/Serialization/QuantizedVectorSerialization.h"
#include "PBDRigidsSolver.h"
#include "Physics/Experimental/PhysScene_Chaos.h"
#include "Physics/NetworkPhysicsComponent.h"


namespace PhysicsReplicationQuantizationCVars
{
	bool bEnabled = false;
	FAutoConsoleVariableRef CVarEnabled(TEXT("np2.Quantization.Enabled"), bEnabled, TEXT("Enable per step physics replication quantization."));

	bool bForceEnabledInStandalone = false;
	FAutoConsoleVariableRef CVarForceEnabledInStandalone(TEXT("np2.Quantization.ForceEnabledInStandalone"), bForceEnabledInStandalone, TEXT("Accept registrations in NM_Standalone worlds."));

	bool bSnapLocation = true;
	FAutoConsoleVariableRef CVarLocation(TEXT("np2.Quantization.Location"), bSnapLocation, TEXT("Per axis snap enable for location."));

	bool bSnapRotation = true;
	FAutoConsoleVariableRef CVarRotation(TEXT("np2.Quantization.Rotation"), bSnapRotation, TEXT("Per axis snap enable for rotation."));

	bool bSnapLinearVel = true;
	FAutoConsoleVariableRef CVarLinearVel(TEXT("np2.Quantization.LinearVel"), bSnapLinearVel, TEXT("Per axis snap enable for linear velocity."));

	bool bSnapAngularVel = true;
	FAutoConsoleVariableRef CVarAngularVel(TEXT("np2.Quantization.AngularVel"), bSnapAngularVel, TEXT("Per axis snap enable for angular velocity."));

	bool bLogEnabled = false;
	FAutoConsoleVariableRef CVarLogEnabled(TEXT("np2.Quantization.Log.Enabled"), bLogEnabled, TEXT("Enable PhysicsReplicationQuantization logging."));

	int32 LogVerbosity = 2;
	FAutoConsoleVariableRef CVarLogVerbosity(TEXT("np2.Quantization.Log.Verbosity"), LogVerbosity, TEXT("PhysicsReplicationQuantization log verbosity, only effective when Log.Enabled is true. 1 = lifecycle, 2 = lifecycle + per step events."));

	bool bLogLocation = true;
	FAutoConsoleVariableRef CVarLogLocationOut(TEXT("np2.Quantization.Log.Location"), bLogLocation, TEXT("Include location (P) in per step snap and Recv logs when LogVerbosity>=2."));

	bool bLogRotation = false;
	FAutoConsoleVariableRef CVarLogRotationOut(TEXT("np2.Quantization.Log.Rotation"), bLogRotation, TEXT("Include rotation (Q) in per step snap and Recv logs when LogVerbosity>=2."));

	bool bLogLinearVel = false;
	FAutoConsoleVariableRef CVarLogLinearVel(TEXT("np2.Quantization.Log.LinearVel"), bLogLinearVel, TEXT("Include linear velocity (V) in per step snap and Recv logs when LogVerbosity>=2."));

	bool bLogAngularVel = false;
	FAutoConsoleVariableRef CVarLogAngularVel(TEXT("np2.Quantization.Log.AngularVel"), bLogAngularVel, TEXT("Include angular velocity (W) in per step snap and Recv logs when LogVerbosity>=2."));

	int32 LogEveryNFrames = 1;
	FAutoConsoleVariableRef CVarLogEveryNFrames(TEXT("np2.Quantization.Log.EveryNFrames"), LogEveryNFrames, TEXT("Emit per step snap and header logs every N solver frames. 1 = every frame, higher is cheaper."));
}

DEFINE_LOG_CATEGORY_STATIC(LogPhysicsReplicationQuantization, Log, All);


// Helpers

FQuantizationAxisGrid VectorQuantizationToGrid(EVectorQuantization Q)
{
	// Mirrors the Iris serializers in PackedVectorNetSerializers.cpp.
	// The "10" and "100" in the enum names are not the actual scale value used, the wire uses power of two scales.
	//   FVectorNetQuantize100NetSerializer: Scale = 1 << 7 = 128
	//   FVectorNetQuantize10NetSerializer:  Scale = 1 << 3 = 8
	//   FVectorNetQuantizeNetSerializer:    Scale = 1 << 0 = 1
	// NOTE: RoundWholeNumber is not used in this per-frame quantization feature since it's too coarse, return Scale=0 instead else it will eat up small movements.
	switch (Q)
	{
	case EVectorQuantization::RoundTwoDecimals: return FQuantizationAxisGrid{ 128, 30, false };
	case EVectorQuantization::RoundOneDecimal: return FQuantizationAxisGrid{ 8, 27, false };
	default: return FQuantizationAxisGrid{ 0, 24, false };
	}
}

static FQuantizationProfile::ERotationMode RotatorQuantizationToMode(ERotatorQuantization R)
{
	// ByteComponents is the engine default so skip that one
	switch (R)
	{
	case ERotatorQuantization::ShortComponents: return FQuantizationProfile::ERotationMode::Euler16Bit;
	default: return FQuantizationProfile::ERotationMode::None;
	}
}

/** Helper function to populate a FQuantizationAxisGrid with the minimum bit size calculated based in the precision wanted and max absolute range */
FQuantizationAxisGrid MakeAxisGrid(double Precision, double MaxAbsRange)
{
	FQuantizationAxisGrid Out;
	Out.bTruncate = false;

	// Largest power of ten that still fits in int32. Anything finer overflows when multiplied by a moderate magnitude value before quantization.
	constexpr int32 MaxScaleCap = 1000000000;

	// Smallest power of 10 such that step (1.0 / Scale) is at most the requested precision.
	int32 Scale = 1;
	const double SafePrecision = FMath::Max(Precision, UE_DOUBLE_KINDA_SMALL_NUMBER);
	while (1.0 / static_cast<double>(Scale) > SafePrecision && Scale < MaxScaleCap)
	{
		Scale *= 10;
	}
	Out.Scale = Scale;

	// Bits sized so a signed integer holds plus or minus MaxAbsRange exactly, plus one for sign.
	const int64 MaxInt = static_cast<int64>(FMath::CeilToDouble(FMath::Abs(MaxAbsRange) * static_cast<double>(Scale)));
	const int32 BitsForMagnitude = (MaxInt > 0) ? FMath::CeilLogTwo64(static_cast<uint64>(MaxInt) + 1) : 0;
	Out.Bits = FMath::Clamp(BitsForMagnitude + 1, 1, 30);

	return Out;
}

FQuantizationProfile BuildProfileFromActor(const AActor* Actor)
{
	FQuantizationProfile Profile;
	if (!Actor)
	{
		return Profile;
	}

	const FRepMovement& Rep = Actor->GetReplicatedMovement();
	Profile.Location        = VectorQuantizationToGrid(Rep.LocationQuantizationLevel);
	Profile.LinearVelocity  = VectorQuantizationToGrid(Rep.VelocityQuantizationLevel);
	Profile.AngularVelocity = Profile.LinearVelocity; // FRepMovement covers linear and angular velocity with one enum.
	Profile.RotationMode    = RotatorQuantizationToMode(Rep.RotationQuantizationLevel);
	return Profile;
}

// FQuat to FRotator and back, the same path FRepMovement uses on the wire.
static FQuat QuantizeRotation(const FQuat& R, FQuantizationProfile::ERotationMode Mode)
{
	if (Mode == FQuantizationProfile::ERotationMode::None)
	{
		return R;
	}

	auto SnapPass = [Mode](const FQuat& InQuat) -> FQuat
	{
		FRotator Rot = InQuat.Rotator();
		if (Mode == FQuantizationProfile::ERotationMode::Euler16Bit)
		{
			const uint16 Pitch = FRotator::CompressAxisToShort(Rot.Pitch);
			const uint16 Yaw   = FRotator::CompressAxisToShort(Rot.Yaw);
			const uint16 Roll  = FRotator::CompressAxisToShort(Rot.Roll);
			Rot.Pitch = FRotator::DecompressAxisFromShort(Pitch);
			Rot.Yaw   = FRotator::DecompressAxisFromShort(Yaw);
			Rot.Roll  = FRotator::DecompressAxisFromShort(Roll);
		}
		else
		{
			const uint8 Pitch = FRotator::CompressAxisToByte(Rot.Pitch);
			const uint8 Yaw   = FRotator::CompressAxisToByte(Rot.Yaw);
			const uint8 Roll  = FRotator::CompressAxisToByte(Rot.Roll);
			Rot.Pitch = FRotator::DecompressAxisFromByte(Pitch);
			Rot.Yaw   = FRotator::DecompressAxisFromByte(Yaw);
			Rot.Roll  = FRotator::DecompressAxisFromByte(Roll);
		}
		return Rot.Quaternion();
	};

	// A single pass through the wire serialize chain leaves the Quat roughly one ULP off the grid at FP precision.
	FQuat Q = SnapPass(R);
	return Q;
}

// In place vector snap. Returns true if the axis was active and snapped.
static bool QuantizeVectorWithGrid(FVector& V, const FQuantizationAxisGrid& Grid)
{
	if (Grid.Scale == 0)
	{
		return false;
	}

	if (Grid.bTruncate)
	{
		// Truncate mode is reserved. Warn once and fall back to round.
		static bool bWarnedTruncate = false;
		if (!bWarnedTruncate)
		{
			bWarnedTruncate = true;
			UE_LOGF(LogPhysicsReplicationQuantization, Warning, "FQuantizationAxisGrid::bTruncate is not implemented. Falling back to round.");
		}
	}

	V = UE::Net::QuantizeVector(Grid.Scale, V);
	return true;
}


// GT shell

namespace
{
	bool ShouldQuantizeInWorld(UWorld* World)
	{
		if (PhysicsReplicationQuantizationCVars::bForceEnabledInStandalone)
		{
			return true;
		}
		return World && World->GetNetMode() != NM_Standalone;
	}

	// NM_Client is client, everything else is treated as server side.
	bool GetIsServerFromScene(FPhysScene_Chaos* Scene)
	{
		if (!Scene)
		{
			return false;
		}
		if (UWorld* World = Scene->GetOwningWorld())
		{
			return World->GetNetMode() != NM_Client;
		}
		return false;
	}
}

FPhysicsReplicationQuantization::FPhysicsReplicationQuantization(FPhysScene_Chaos* InPhysicsScene)
	: PhysicsScene(InPhysicsScene)
{
	check(PhysicsScene);

	if (Chaos::FPhysicsSolver* Solver = PhysicsScene->GetSolver())
	{
		AsyncCallback = Solver->CreateAndRegisterSimCallbackObject_External<FPhysicsReplicationQuantizationAsync>();
	}

	// Subscribe to a tick to populate AsyncInput
	PreTickHandle = PhysicsScene->OnPhysScenePreTick.AddRaw(this, &FPhysicsReplicationQuantization::OnPhysScenePreTick);

	// Seed the values on the PT side. World may not be set yet, the pre tick callback will push later if anything changed.
	PushPerTickStateIfChanged();
}

FPhysicsReplicationQuantization::~FPhysicsReplicationQuantization()
{
	if (PhysicsScene && PreTickHandle.IsValid())
	{
		PhysicsScene->OnPhysScenePreTick.Remove(PreTickHandle);
		PreTickHandle.Reset();
	}

	if (AsyncCallback && PhysicsScene)
	{
		if (Chaos::FPhysicsSolver* Solver = PhysicsScene->GetSolver())
		{
			Solver->UnregisterAndFreeSimCallbackObject_External(AsyncCallback);
		}
	}
	AsyncCallback = nullptr;
}

void FPhysicsReplicationQuantization::OnPhysScenePreTick(FPhysScene_Chaos* /*Scene*/, float /*DeltaSeconds*/)
{
	PushPerTickStateIfChanged();
}

void FPhysicsReplicationQuantization::PushPerTickStateIfChanged()
{
	if (!AsyncCallback)
	{
		return;
	}

	const bool bIsServer = GetIsServerFromScene(PhysicsScene);

	if (bHasPushedPerTickStateOnce && bIsServer == bLastPushedIsServer)
	{
		return;
	}

	if (FQuantizationAsyncInput* AsyncInput = AsyncCallback->GetProducerInputData_External())
	{
		AsyncInput->SetIsServer = bIsServer;
		bLastPushedIsServer = bIsServer;
		bHasPushedPerTickStateOnce = true;
	}
}

Chaos::FConstPhysicsObjectHandle FPhysicsReplicationQuantization::GetActorHandle(AActor* Actor) const
{
	if (!Actor)
	{
		return nullptr;
	}
	UPrimitiveComponent* Root = Cast<UPrimitiveComponent>(Actor->GetRootComponent());
	if (!Root)
	{
		return nullptr;
	}
	return Root->GetPhysicsObjectByName(NAME_None);
}

void FPhysicsReplicationQuantization::RegisterActor(AActor* Actor)
{
	if (!AsyncCallback || !Actor)
	{
		return;
	}

	if (!ShouldQuantizeInWorld(Actor->GetWorld()))
	{
		return;
	}

	const FQuantizationProfile Profile = BuildProfileFromActor(Actor);
	if (!Profile.HasAnyAxis())
	{
		if (PhysicsReplicationQuantizationCVars::bLogEnabled && PhysicsReplicationQuantizationCVars::LogVerbosity >= 1)
		{
			UE_LOGF(LogPhysicsReplicationQuantization, Log, "RegisterActor %ls skipped, FRepMovement is at engine defaults (no opt up).", *Actor->GetName());
		}
		return;
	}

	const Chaos::FConstPhysicsObjectHandle Handle = GetActorHandle(Actor);
	if (!Handle)
	{
		return;
	}

	if (FQuantizationAsyncInput* AsyncInput = AsyncCallback->GetProducerInputData_External())
	{
		FQuantizationAsyncInput::FAdd& Add = AsyncInput->Adds.AddDefaulted_GetRef();
		Add.Handle = Handle;
		Add.Profile = Profile;
		Add.DebugName = Actor->GetName();
	}

	ActorHandles_External.Add(Actor, Handle);

	if (PhysicsReplicationQuantizationCVars::bLogEnabled && PhysicsReplicationQuantizationCVars::LogVerbosity >= 1)
	{
		UE_LOGF(LogPhysicsReplicationQuantization, Log, "RegisterActor %ls -> handle %p (LocScale=%d RotMode=%d VelScale=%d)",
			*Actor->GetName(), Handle, Profile.Location.Scale, (int32)Profile.RotationMode, Profile.LinearVelocity.Scale);
	}
}

void FPhysicsReplicationQuantization::RefreshActor(AActor* Actor)
{
	// Re register with a freshly built profile, the PT side map overwrites.
	RegisterActor(Actor);
}

void FPhysicsReplicationQuantization::UnregisterActor(AActor* Actor)
{
	if (!AsyncCallback || !Actor)
	{
		return;
	}

	Chaos::FConstPhysicsObjectHandle Handle = nullptr;
	if (Chaos::FConstPhysicsObjectHandle* Found = ActorHandles_External.Find(Actor))
	{
		Handle = *Found;
		ActorHandles_External.Remove(Actor);
	}
	else
	{
		// Actor was never registered, nothing to do.
		return;
	}

	if (FQuantizationAsyncInput* AsyncInput = AsyncCallback->GetProducerInputData_External())
	{
		AsyncInput->Removes.Add(Handle);
	}

	if (PhysicsReplicationQuantizationCVars::bLogEnabled && PhysicsReplicationQuantizationCVars::LogVerbosity >= 1)
	{
		UE_LOGF(LogPhysicsReplicationQuantization, Log, "UnregisterActor %ls -> handle %p", *Actor->GetName(), Handle);
	}
}

void FPhysicsReplicationQuantization::RegisterCustomProfile(Chaos::FConstPhysicsObjectHandle Handle, const FQuantizationProfile& Profile, const FString& DebugName)
{
	if (!AsyncCallback || !Handle || !Profile.HasAnyAxis())
	{
		return;
	}

	if (PhysicsScene && !ShouldQuantizeInWorld(PhysicsScene->GetOwningWorld()))
	{
		return;
	}

	if (FQuantizationAsyncInput* AsyncInput = AsyncCallback->GetProducerInputData_External())
	{
		FQuantizationAsyncInput::FAdd& Add = AsyncInput->Adds.AddDefaulted_GetRef();
		Add.Handle = Handle;
		Add.Profile = Profile;
		Add.DebugName = DebugName;
	}
}

void FPhysicsReplicationQuantization::RefreshCustomProfile(Chaos::FConstPhysicsObjectHandle Handle, const FQuantizationProfile& Profile, const FString& DebugName)
{
	RegisterCustomProfile(Handle, Profile, DebugName);
}

void FPhysicsReplicationQuantization::UnregisterHandle(Chaos::FConstPhysicsObjectHandle Handle)
{
	if (!AsyncCallback || !Handle)
	{
		return;
	}

	if (FQuantizationAsyncInput* AsyncInput = AsyncCallback->GetProducerInputData_External())
	{
		AsyncInput->Removes.Add(Handle);
	}
}


// Component lifecycle helper (GT)

void UE::PhysicsReplicationQuantization::NotifyComponentPhysicsState(UPrimitiveComponent* Self, bool bRegister)
{
	if (!Self)
	{
		return;
	}
	AActor* Owner = Self->GetOwner();
	if (!Owner || Owner->GetRootComponent() != Self)
	{
		// Only the actor's root primitive drives registration.
		return;
	}
	if (bRegister && !Owner->IsReplicatingMovement())
	{
		// SetReplicateMovement will register us later if the flag flips.
		return;
	}
	UWorld* World = Self->GetWorld();
	if (!World)
	{
		return;
	}
	FPhysScene_Chaos* Scene = static_cast<FPhysScene_Chaos*>(World->GetPhysicsScene());
	if (!Scene)
	{
		return;
	}
	FPhysicsReplicationQuantization* PhysRepQuantization = Scene->GetPhysicsReplicationQuantization();
	if (!PhysRepQuantization)
	{
		return;
	}
	if (bRegister)
	{
		PhysRepQuantization->RegisterActor(Owner);
	}
	else
	{
		PhysRepQuantization->UnregisterActor(Owner);
	}
}


// Static PT side accessors

FPhysicsReplicationQuantizationAsync* UE::PhysicsReplicationQuantization::GetAsyncFromSolver(Chaos::FPhysicsSolverBase* Solver)
{
	if (!Solver)
	{
		return nullptr;
	}
	Chaos::FPBDRigidsSolver* RigidsSolver = static_cast<Chaos::FPBDRigidsSolver*>(Solver);
	FPhysScene_Chaos* Scene = static_cast<FPhysScene_Chaos*>(RigidsSolver->PhysSceneHack);
	if (!Scene)
	{
		return nullptr;
	}
	FPhysicsReplicationQuantization* PhysRepQuantization = Scene->GetPhysicsReplicationQuantization();
	return PhysRepQuantization ? PhysRepQuantization->GetAsync() : nullptr;
}

bool UE::PhysicsReplicationQuantization::IsHandleManaged_Internal(Chaos::FPhysicsSolverBase* Solver, Chaos::FConstPhysicsObjectHandle Handle)
{
	if (!Handle)
	{
		return false;
	}
	const FPhysicsReplicationQuantizationAsync* Async = GetAsyncFromSolver(Solver);
	return Async && Async->IsHandleManaged_Internal(Handle);
}

void UE::PhysicsReplicationQuantization::ForceSnapAllRegistered_Internal(Chaos::FPhysicsSolverBase* Solver, bool bIncludeCurrentXR)
{
	if (FPhysicsReplicationQuantizationAsync* Async = GetAsyncFromSolver(Solver))
	{
		Async->ForceSnapAllRegistered_Internal(bIncludeCurrentXR, /*bAllowLogging=*/false);
	}
}

bool UE::PhysicsReplicationQuantization::VerifyTargetOnGrid_Internal(
	Chaos::FPhysicsSolverBase* Solver,
	Chaos::FConstPhysicsObjectHandle Handle,
	int32 ServerFrame,
	const FVector& TargetPosition,
	const FQuat& TargetRotation,
	const FVector& TargetLinVel,
	const FVector& TargetAngVelDeg)
{
	if (!Handle)
	{
		return true;
	}
	FPhysicsReplicationQuantizationAsync* Async = GetAsyncFromSolver(Solver);
	return !Async || Async->VerifyTargetOnGrid_Internal(Handle, ServerFrame, TargetPosition, TargetRotation, TargetLinVel, TargetAngVelDeg);
}


// PT sim callback

bool FPhysicsReplicationQuantizationAsync::IsHandleManaged_Internal(Chaos::FConstPhysicsObjectHandle Handle) const
{
	return PhysicsReplicationQuantizationCVars::bEnabled && Profiles_Internal.Contains(Handle);
}

bool FPhysicsReplicationQuantizationAsync::IsHandleRegistered_Internal(Chaos::FConstPhysicsObjectHandle Handle) const
{
	return Profiles_Internal.Contains(Handle);
}

namespace
{
	// Number of non verified or mismatched targets we accept before concluding the network is on a coarser strict subset of our grid.
	// Each non discriminating sample cuts the false positive odds in half
	constexpr uint32 GVerifyConcludeAfterSamples = 30;

	// Shared implementation for any vector grid axis class (Location, LinVel, AngVel).
	// Mutates State and TestsAttempted on the slot. Returns false on detected mismatch.
	bool VerifyVectorAxis(
		const TCHAR* NameStr, Chaos::FConstPhysicsObjectHandle Handle, const TCHAR* AxisLabel,
		int32 Scale, const FVector& Value,
		FPhysicsReplicationQuantizationAsync::EVerificationState& State, uint32& TestsAttempted)
	{
		using EState = FPhysicsReplicationQuantizationAsync::EVerificationState;

		if (Scale == 0 || State != EState::Unverified)
		{
			return true;
		}
		if (Value.IsNearlyZero())
		{
			return true;
		}

		TestsAttempted++;

		auto ScaleOf = [Scale](double V) -> double { return V * static_cast<double>(Scale); };
		auto OnGrid = [&ScaleOf](double V) -> bool
		{
			const double Scaled = ScaleOf(V);
			const double Rounded = FMath::RoundToDouble(Scaled);
			return FMath::Abs(Scaled - Rounded) <= 1e-3;
		};

		if (!OnGrid(Value.X) || !OnGrid(Value.Y) || !OnGrid(Value.Z))
		{
			State = EState::MismatchReported;
			const double SX = ScaleOf(Value.X);
			const double SY = ScaleOf(Value.Y);
			const double SZ = ScaleOf(Value.Z);
			UE_LOGF(LogPhysicsReplicationQuantization, Error,
				"[PRQ] %ls wire grid drift for %ls (handle %p): target (%.10f, %.10f, %.10f) is not on the assumed 1/%d grid. Scaled values: (%.6f, %.6f, %.6f).",
				AxisLabel, NameStr, Handle, Value.X, Value.Y, Value.Z, Scale, SX, SY, SZ);
			ensureMsgf(false, TEXT("[PRQ] %s wire grid drift (Scale=%d)"), AxisLabel, Scale);
			return false;
		}

		// Discriminating: low bit of any scaled int component is set, meaning the value lies on 1/Scale but not on the coarser 1/(Scale/2) subset.
		const int64 IX = static_cast<int64>(FMath::RoundToDouble(ScaleOf(Value.X)));
		const int64 IY = static_cast<int64>(FMath::RoundToDouble(ScaleOf(Value.Y)));
		const int64 IZ = static_cast<int64>(FMath::RoundToDouble(ScaleOf(Value.Z)));
		if (((IX | IY | IZ) & int64(1)) != 0)
		{
			State = EState::Verified;
			if (PhysicsReplicationQuantizationCVars::bLogEnabled && PhysicsReplicationQuantizationCVars::LogVerbosity >= 1)
			{
				UE_LOGF(LogPhysicsReplicationQuantization, Log, "[PRQ] %ls wire grid verified for %ls (handle %p, Scale=%d)", AxisLabel, NameStr, Handle, Scale);
			}
		}
		return true;
	}
}

bool FPhysicsReplicationQuantizationAsync::VerifyTargetOnGrid_Internal(
	Chaos::FConstPhysicsObjectHandle Handle, int32 ServerFrame,
	const FVector& TargetPosition, const FQuat& TargetRotation,
	const FVector& TargetLinVel, const FVector& TargetAngVelDeg)
{
	if (!PhysicsReplicationQuantizationCVars::bEnabled)
	{
		return true;
	}

	FProfileSlot* Slot = Profiles_Internal.Find(Handle);
	if (!Slot)
	{
		return true;
	}

	const TCHAR* NameStr = Slot->DebugName.IsEmpty() ? TEXT("?") : *Slot->DebugName;
	bool bAllOk = true;

	// Per axis "received target" log lines, gated like the PostSolve snap log.
	// ServerFrame is aligned back by one so Recv[N] sits on the same column as PostSolve[N] for the same physical state.
	if (PhysicsReplicationQuantizationCVars::bLogEnabled && PhysicsReplicationQuantizationCVars::LogVerbosity >= 2)
	{
		const bool bLogP = PhysicsReplicationQuantizationCVars::bLogLocation && Slot->Profile.Location.Scale != 0;
		const bool bLogQ = PhysicsReplicationQuantizationCVars::bLogRotation && Slot->Profile.RotationMode != FQuantizationProfile::ERotationMode::None;
		const bool bLogV = PhysicsReplicationQuantizationCVars::bLogLinearVel && Slot->Profile.LinearVelocity.Scale != 0;
		const bool bLogW = PhysicsReplicationQuantizationCVars::bLogAngularVel && Slot->Profile.AngularVelocity.Scale != 0;
		if (bLogP || bLogQ || bLogV || bLogW)
		{
			const int32 AlignedServerFrame = ServerFrame - 1;
			const int32 LocalFrame = AlignedServerFrame - UE::NetworkPhysicsUtils::GetNetworkPhysicsTickOffset_Internal(GetSolver());
			const TCHAR* RoleStr = bIsServer_Internal ? TEXT("Server") : TEXT("Client");

			if (bLogP)
			{
				UE_LOGF(LogPhysicsReplicationQuantization, Log,
					"\t[PRQ] Recv LocalFrame=%d ServerFrame=%d Role=%ls Name=%ls Handle=%p P: target=(%.6f,%.6f,%.6f)",
					LocalFrame, AlignedServerFrame, RoleStr, NameStr, Handle, TargetPosition.X, TargetPosition.Y, TargetPosition.Z);
			}
			if (bLogQ)
			{
				const FQuat QLog = CanonicalizeQuatForLog(TargetRotation);
				UE_LOGF(LogPhysicsReplicationQuantization, Log,
					"\t[PRQ] Recv LocalFrame=%d ServerFrame=%d Role=%ls Name=%ls Handle=%p Q(xyzw): target=(%.9f,%.9f,%.9f,%.9f)",
					LocalFrame, AlignedServerFrame, RoleStr, NameStr, Handle, QLog.X, QLog.Y, QLog.Z, QLog.W);
			}
			if (bLogV)
			{
				UE_LOGF(LogPhysicsReplicationQuantization, Log,
					"\t[PRQ] Recv LocalFrame=%d ServerFrame=%d Role=%ls Name=%ls Handle=%p V: target=(%.6f,%.6f,%.6f)",
					LocalFrame, AlignedServerFrame, RoleStr, NameStr, Handle, TargetLinVel.X, TargetLinVel.Y, TargetLinVel.Z);
			}
			if (bLogW)
			{
				UE_LOGF(LogPhysicsReplicationQuantization, Log,
					"\t[PRQ] Recv LocalFrame=%d ServerFrame=%d Role=%ls Name=%ls Handle=%p W(deg/s): target=(%.6f,%.6f,%.6f)",
					LocalFrame, AlignedServerFrame, RoleStr, NameStr, Handle, TargetAngVelDeg.X, TargetAngVelDeg.Y, TargetAngVelDeg.Z);
			}
		}
	}

	// Location, LinVel, AngVel share vector grid logic.
	bAllOk &= VerifyVectorAxis(NameStr, Handle, TEXT("Location"), Slot->Profile.Location.Scale, TargetPosition, Slot->LocationVerifyState, Slot->LocationTestsAttempted);
	bAllOk &= VerifyVectorAxis(NameStr, Handle, TEXT("LinVel"), Slot->Profile.LinearVelocity.Scale, TargetLinVel, Slot->LinVelVerifyState, Slot->LinVelTestsAttempted);
	bAllOk &= VerifyVectorAxis(NameStr, Handle, TEXT("AngVel"), Slot->Profile.AngularVelocity.Scale, TargetAngVelDeg, Slot->AngVelVerifyState, Slot->AngVelTestsAttempted);

	// Rotation uses its own grid check via compress / decompress round trip.
	const FQuantizationProfile::ERotationMode RotMode = Slot->Profile.RotationMode;
	if (RotMode != FQuantizationProfile::ERotationMode::None && Slot->RotationVerifyState == EVerificationState::Unverified)
	{
		// Identity quaternion is the rotation analogue of zero position, on every grid.
		const FRotator Eul = TargetRotation.Rotator();
		if (!FMath::IsNearlyZero(Eul.Pitch) || !FMath::IsNearlyZero(Eul.Yaw) || !FMath::IsNearlyZero(Eul.Roll))
		{
			Slot->RotationTestsAttempted++;

			// Grid step distance is wrap invariant, [0,360) versus [-180,180] both work.
			const double Step = (RotMode == FQuantizationProfile::ERotationMode::Euler16Bit) ? (360.0 / 65536.0) : (360.0 / 256.0);
			auto OnGrid = [Step](double Deg) -> bool
			{
				const double Scaled  = Deg / Step;
				const double Rounded = FMath::RoundToDouble(Scaled);
				return FMath::Abs(Scaled - Rounded) <= 1e-2;
			};

			if (!OnGrid(Eul.Pitch) || !OnGrid(Eul.Yaw) || !OnGrid(Eul.Roll))
			{
				Slot->RotationVerifyState = EVerificationState::MismatchReported;
				UE_LOGF(LogPhysicsReplicationQuantization, Error,
					"[PRQ] Rotation wire grid drift for %ls (handle %p). Target euler (%.6f, %.6f, %.6f) does not round trip through compress and decompress (mode=%d).",
					NameStr, Handle, Eul.Pitch, Eul.Yaw, Eul.Roll, (int32)RotMode);
				ensureMsgf(false, TEXT("[PRQ] Rotation wire grid drift (Mode=%d)"), (int32)RotMode);
				bAllOk = false;
			}
			else
			{
				// For 16Bit, require at least one compressed value with low 8 bits non zero, otherwise the value also lies on the 8Bit grid (65536/256 = 256).
				// For 8Bit there is no coarser subset to disambiguate against.
				bool bDiscriminating = true;
				if (RotMode == FQuantizationProfile::ERotationMode::Euler16Bit)
				{
					const uint16 P = FRotator::CompressAxisToShort(Eul.Pitch);
					const uint16 Y = FRotator::CompressAxisToShort(Eul.Yaw);
					const uint16 R = FRotator::CompressAxisToShort(Eul.Roll);
					bDiscriminating = ((P | Y | R) & uint16(0xFF)) != 0;
				}

				if (bDiscriminating)
				{
					Slot->RotationVerifyState = EVerificationState::Verified;
					if (PhysicsReplicationQuantizationCVars::bLogEnabled && PhysicsReplicationQuantizationCVars::LogVerbosity >= 1)
					{
						UE_LOGF(LogPhysicsReplicationQuantization, Log, "[PRQ] Rotation wire grid verified for %ls (handle %p, Mode=%d)", NameStr, Handle, (int32)RotMode);
					}
				}
			}
		}
	}

	return bAllOk;
}

void FPhysicsReplicationQuantizationAsync::NotifySlotPreRemove_Internal(Chaos::FConstPhysicsObjectHandle Handle)
{
	const FProfileSlot* Slot = Profiles_Internal.Find(Handle);
	if (!Slot)
	{
		return;
	}

	const TCHAR* NameStr = Slot->DebugName.IsEmpty() ? TEXT("?") : *Slot->DebugName;

	auto ReportIfStuck = [&](EVerificationState State, uint32 Tests, const TCHAR* AxisClass, int32 ScaleOrMode)
	{
		if (State != EVerificationState::Unverified || Tests < GVerifyConcludeAfterSamples)
		{
			return;
		}

		const FString Key = FString::Printf(TEXT("%s|%s"), *Slot->DebugName, AxisClass);
		if (UnverifiedReportedNames_Internal.Contains(Key))
		{
			return;
		}
		UnverifiedReportedNames_Internal.Add(Key);

		UE_LOGF(LogPhysicsReplicationQuantization, Error,
			"[PRQ] %ls wire grid never discriminated for %ls after %u non trivial targets. The network is likely using a coarser quantization than the actor's profile claims (assumed 1/%d).",
			AxisClass, NameStr, Tests, ScaleOrMode);
		ensureMsgf(false, TEXT("[PRQ] %s grid never discriminated (likely coarser wire)"), AxisClass);
	};

	ReportIfStuck(Slot->LocationVerifyState, Slot->LocationTestsAttempted, TEXT("Location"), Slot->Profile.Location.Scale);
	ReportIfStuck(Slot->RotationVerifyState, Slot->RotationTestsAttempted, TEXT("Rotation"), (int32)Slot->Profile.RotationMode);
	ReportIfStuck(Slot->LinVelVerifyState, Slot->LinVelTestsAttempted, TEXT("LinVel"), Slot->Profile.LinearVelocity.Scale);
	ReportIfStuck(Slot->AngVelVerifyState, Slot->AngVelTestsAttempted, TEXT("AngVel"), Slot->Profile.AngularVelocity.Scale);
}

void FPhysicsReplicationQuantizationAsync::ProcessAsyncInputs()
{
	if (const FQuantizationAsyncInput* Input = GetConsumerInput_Internal())
	{
		for (const FQuantizationAsyncInput::FAdd& Add : Input->Adds)
		{
			if (Add.Handle)
			{
				FProfileSlot& Slot = Profiles_Internal.FindOrAdd(Add.Handle);
				Slot.Profile = Add.Profile;
				if (!Add.DebugName.IsEmpty())
				{
					Slot.DebugName = Add.DebugName;
				}
			}
		}
		for (Chaos::FConstPhysicsObjectHandle Handle : Input->Removes)
		{
			NotifySlotPreRemove_Internal(Handle);
			Profiles_Internal.Remove(Handle);
		}
		if (Input->SetIsServer.IsSet())
		{
			bIsServer_Internal = *Input->SetIsServer;
		}
	}
}

void FPhysicsReplicationQuantizationAsync::OnPreSimulate_Internal()
{
	// Apply Adds and Removes early so other PostSolve callbacks see current registration state.
	ProcessAsyncInputs();
}

void FPhysicsReplicationQuantizationAsync::FirstPreResimStep_Internal(int32 /*PhysicsStep*/)
{
	// FRewindData::ApplyTargets just wrote the cached server target onto the particle. The network received Quat is roughly one ULP off the deterministic local snap
	// Force snap before AdvanceSolver runs any other sim callback. XR included because ApplyTargets writes to XR
	// FrameLabelDelta = -1 so the log stamps end of step (N-1), matching the ServerFrame=K = end of step K convention used by PostSolve and Recv
	EmitCallerHeaderLog_Internal(TEXT("FirstPreResimStep"), /*FrameLabelDelta=*/-1);
	ForceSnapAllRegistered_Internal(/*bIncludeCurrentXR=*/true, /*bAllowLogging=*/true, /*FrameLabelDelta=*/-1);
}

void FPhysicsReplicationQuantizationAsync::OnPostSolve_Internal()
{
	// Per step snap of P, Q, V, W
	EmitCallerHeaderLog_Internal(TEXT("PostSolve"));
	ForceSnapAllRegistered_Internal(/*bIncludeCurrentXR=*/false, /*bAllowLogging=*/true);
}

void FPhysicsReplicationQuantizationAsync::OnPhysicsObjectUnregistered_Internal(Chaos::FConstPhysicsObjectHandle PhysicsObject)
{
	NotifySlotPreRemove_Internal(PhysicsObject);
	Profiles_Internal.Remove(PhysicsObject);
}

void FPhysicsReplicationQuantizationAsync::EmitCallerHeaderLog_Internal(const TCHAR* CallerName, int32 FrameLabelDelta)
{
	if (!PhysicsReplicationQuantizationCVars::bLogEnabled || PhysicsReplicationQuantizationCVars::LogVerbosity < 2)
	{
		return;
	}
	if (!PhysicsReplicationQuantizationCVars::bLogLocation
		&& !PhysicsReplicationQuantizationCVars::bLogRotation
		&& !PhysicsReplicationQuantizationCVars::bLogLinearVel
		&& !PhysicsReplicationQuantizationCVars::bLogAngularVel)
	{
		return;
	}

	int32 PhysicsFrame = INDEX_NONE;
	int32 ServerFrame = INDEX_NONE;
	bool bResim = false;
	if (Chaos::FPBDRigidsSolver* RigidsSolver = static_cast<Chaos::FPBDRigidsSolver*>(GetSolver()))
	{
		PhysicsFrame = RigidsSolver->GetCurrentFrame() + FrameLabelDelta;
		ServerFrame = PhysicsFrame + UE::NetworkPhysicsUtils::GetNetworkPhysicsTickOffset_Internal(RigidsSolver);
		if (RigidsSolver->GetEvolution())
		{
			bResim = RigidsSolver->GetEvolution()->IsResimming();
		}
	}

	const int32 EveryN = FMath::Max(PhysicsReplicationQuantizationCVars::LogEveryNFrames, 1);
	if (EveryN > 1 && (PhysicsFrame % EveryN) != 0)
	{
		return;
	}

	const TCHAR* RoleStr = bIsServer_Internal ? TEXT("Server") : TEXT("Client");

	UE_LOGF(LogPhysicsReplicationQuantization, Log, "[PRQ] Caller=%ls LocalFrame=%d ServerFrame=%d Role=%ls Resim=%d",
		CallerName, PhysicsFrame, ServerFrame, RoleStr, bResim ? 1 : 0);
}

void FPhysicsReplicationQuantizationAsync::ForceSnapAllRegistered_Internal(bool bIncludeCurrentXR, bool bAllowLogging, int32 FrameLabelDelta)
{
	const bool bSnapEnabled = PhysicsReplicationQuantizationCVars::bEnabled;

	// Caller controls bAllowLogging so the public namespace wrapper emits no per step lines regardless of CVar state.
	bool bLogP = false;
	bool bLogQ = false;
	bool bLogV = false;
	bool bLogW = false;
	if (bAllowLogging && PhysicsReplicationQuantizationCVars::bLogEnabled && PhysicsReplicationQuantizationCVars::LogVerbosity >= 2)
	{
		bLogP = PhysicsReplicationQuantizationCVars::bLogLocation;
		bLogQ = PhysicsReplicationQuantizationCVars::bLogRotation;
		bLogV = PhysicsReplicationQuantizationCVars::bLogLinearVel;
		bLogW = PhysicsReplicationQuantizationCVars::bLogAngularVel;
	}
	const bool bAnyLog = bLogP || bLogQ || bLogV || bLogW;
	if (!bSnapEnabled && !bAnyLog)
	{
		return;
	}

	const bool bCVarLoc = PhysicsReplicationQuantizationCVars::bSnapLocation;
	const bool bCVarRot = PhysicsReplicationQuantizationCVars::bSnapRotation;
	const bool bCVarLin = PhysicsReplicationQuantizationCVars::bSnapLinearVel;
	const bool bCVarAng = PhysicsReplicationQuantizationCVars::bSnapAngularVel;

	// Resim flag is stamped on every log line so forward pass values (Resim=0) and resim values (Resim=1) can be told apart in the analyzer.
	int32 PhysicsFrame = INDEX_NONE;
	int32 ServerFrame = INDEX_NONE;
	bool bResim = false;
	if (bAnyLog)
	{
		if (Chaos::FPBDRigidsSolver* RigidsSolver = static_cast<Chaos::FPBDRigidsSolver*>(GetSolver()))
		{
			PhysicsFrame = RigidsSolver->GetCurrentFrame() + FrameLabelDelta;
			// Server offset is 0. Adding it on the client aligns log lines to the server's CurrentFrame so the two ends can be matched on ServerFrame.
			ServerFrame = PhysicsFrame + UE::NetworkPhysicsUtils::GetNetworkPhysicsTickOffset_Internal(RigidsSolver);
			if (RigidsSolver->GetEvolution())
			{
				bResim = RigidsSolver->GetEvolution()->IsResimming();
			}
		}
	}

	// Throttle the per step log emission. The snap still runs every frame regardless.
	if (bAnyLog)
	{
		const int32 EveryN = FMath::Max(PhysicsReplicationQuantizationCVars::LogEveryNFrames, 1);
		if (EveryN > 1 && (PhysicsFrame % EveryN) != 0)
		{
			bLogP = bLogQ = bLogV = bLogW = false;
		}
	}

	if (!bSnapEnabled && !bLogP && !bLogQ && !bLogV && !bLogW)
	{
		return;
	}

	const int32 ResimFlag = bResim ? 1 : 0;
	const TCHAR* RoleStr = bIsServer_Internal ? TEXT("Server") : TEXT("Client");

	Chaos::FReadPhysicsObjectInterface_Internal Interface = Chaos::FPhysicsObjectInternalInterface::GetRead();

	for (const TPair<Chaos::FConstPhysicsObjectHandle, FProfileSlot>& Pair : Profiles_Internal)
	{
		Chaos::FPBDRigidParticleHandle* Rigid = Interface.GetRigidParticle(Pair.Key);
		if (!Rigid)
		{
			continue;
		}

		// Static and kinematic particles are not integrated, nothing to snap.
		const Chaos::EObjectStateType State = Rigid->ObjectState();
		if (State != Chaos::EObjectStateType::Dynamic && State != Chaos::EObjectStateType::Sleeping)
		{
			continue;
		}

		const FQuantizationProfile& Profile = Pair.Value.Profile;
		const TCHAR* NameStr = Pair.Value.DebugName.IsEmpty() ? TEXT("?") : *Pair.Value.DebugName;

		// Per axis: snap when bSnapEnabled and the per axis CVar are both on, logs fire per axis under the Log.<X> CVars.
		// When the snap is off the log still emits, post == pre and diff is zero, which is fine for the analyzer.
		// X and R are snapped in addition to P and Q only when bIncludeCurrentXR is true (resim entry).

		if (Profile.Location.Scale != 0)
		{
			// Read and write P, not X. Downstream Chaos passes act on P and Q. X is also snapped on resim entry because ApplyTargets writes both pairs.
			const FVector PreP = Rigid->GetP();
			FVector P = PreP;
			if (bSnapEnabled && bCVarLoc)
			{
				if (QuantizeVectorWithGrid(P, Profile.Location))
				{
					Rigid->SetP(P);
				}
				if (bIncludeCurrentXR)
				{
					FVector X = Rigid->GetX();
					if (QuantizeVectorWithGrid(X, Profile.Location))
					{
						Rigid->SetX(X);
					}
				}
			}
			if (bLogP)
			{
				// Read back from the particle so the log reflects what is actually stored.
				const FVector PostP = Rigid->GetP();
				const FVector DiffP = PostP - PreP;
				UE_LOGF(LogPhysicsReplicationQuantization, Log,
					"\t[PRQ] LocalFrame=%d ServerFrame=%d Role=%ls Resim=%d Name=%ls Handle=%p P: pre=(%.6f,%.6f,%.6f) post=(%.6f,%.6f,%.6f) diff=(%.6f,%.6f,%.6f) |diff|=%.6f",
					PhysicsFrame, ServerFrame, RoleStr, ResimFlag, NameStr, Pair.Key,
					PreP.X, PreP.Y, PreP.Z, PostP.X, PostP.Y, PostP.Z, DiffP.X, DiffP.Y, DiffP.Z, DiffP.Size());
			}
		}

		if (Profile.RotationMode != FQuantizationProfile::ERotationMode::None)
		{
			// Same predicted vs current rule as P, the advance state pass copies Q to R between frames.
			const FQuat PreQ = Rigid->GetQ();
			if (bSnapEnabled && bCVarRot)
			{
				const FQuat PostQ = QuantizeRotation(PreQ, Profile.RotationMode);
				Rigid->SetQ(PostQ);
				if (bIncludeCurrentXR)
				{
					Rigid->SetR(QuantizeRotation(Rigid->GetR(), Profile.RotationMode));
				}
			}
			if (bLogQ)
			{
				// Raw Quat components, the Quat / Rotator conversion is FP lossy and would obscure the stored value diff. Sign canonicalized so client and server lines compare cleanly.
				const FQuat PreQLog = CanonicalizeQuatForLog(PreQ);
				const FQuat PostQLog = CanonicalizeQuatForLog(Rigid->GetQ());
				const FQuat DiffQ(PostQLog.X - PreQLog.X, PostQLog.Y - PreQLog.Y, PostQLog.Z - PreQLog.Z, PostQLog.W - PreQLog.W);
				UE_LOGF(LogPhysicsReplicationQuantization, Log,
					"\t[PRQ] LocalFrame=%d ServerFrame=%d Role=%ls Resim=%d Name=%ls Handle=%p Q(xyzw): pre=(%.9f,%.9f,%.9f,%.9f) post=(%.9f,%.9f,%.9f,%.9f) diff=(%.9f,%.9f,%.9f,%.9f)",
					PhysicsFrame, ServerFrame, RoleStr, ResimFlag, NameStr, Pair.Key,
					PreQLog.X, PreQLog.Y, PreQLog.Z, PreQLog.W, PostQLog.X, PostQLog.Y, PostQLog.Z, PostQLog.W, DiffQ.X, DiffQ.Y, DiffQ.Z, DiffQ.W);
			}
		}

		if (Profile.LinearVelocity.Scale != 0)
		{
			const FVector PreV = Rigid->GetV();
			FVector V = PreV;
			if (bSnapEnabled && bCVarLin)
			{
				if (QuantizeVectorWithGrid(V, Profile.LinearVelocity))
				{
					Rigid->SetV(V);
				}
			}
			if (bLogV)
			{
				const FVector PostV = Rigid->GetV();
				const FVector DiffV = PostV - PreV;
				UE_LOGF(LogPhysicsReplicationQuantization, Log,
					"\t[PRQ] LocalFrame=%d ServerFrame=%d Role=%ls Resim=%d Name=%ls Handle=%p V: pre=(%.6f,%.6f,%.6f) post=(%.6f,%.6f,%.6f) diff=(%.6f,%.6f,%.6f) |diff|=%.6f",
					PhysicsFrame, ServerFrame, RoleStr, ResimFlag, NameStr, Pair.Key,
					PreV.X, PreV.Y, PreV.Z, PostV.X, PostV.Y, PostV.Z, DiffV.X, DiffV.Y, DiffV.Z, DiffV.Size());
			}
		}

		if (Profile.AngularVelocity.Scale != 0)
		{
			// W is in rad/s on the particle, the wire format quantizes deg/s.
			const FVector PreWDeg = FMath::RadiansToDegrees(Rigid->GetW());
			FVector WDeg = PreWDeg;
			if (bSnapEnabled && bCVarAng)
			{
				if (QuantizeVectorWithGrid(WDeg, Profile.AngularVelocity))
				{
					Rigid->SetW(FMath::DegreesToRadians(WDeg));
				}
			}
			if (bLogW)
			{
				const FVector PostWDeg = FMath::RadiansToDegrees(Rigid->GetW());
				const FVector DiffWDeg = PostWDeg - PreWDeg;
				UE_LOGF(LogPhysicsReplicationQuantization, Log,
					"\t[PRQ] LocalFrame=%d ServerFrame=%d Role=%ls Resim=%d Name=%ls Handle=%p W(deg/s): pre=(%.6f,%.6f,%.6f) post=(%.6f,%.6f,%.6f) diff=(%.6f,%.6f,%.6f) |diff|=%.6f",
					PhysicsFrame, ServerFrame, RoleStr, ResimFlag, NameStr, Pair.Key,
					PreWDeg.X, PreWDeg.Y, PreWDeg.Z, PostWDeg.X, PostWDeg.Y, PostWDeg.Z, DiffWDeg.X, DiffWDeg.Y, DiffWDeg.Z, DiffWDeg.Size());
			}
		}
	}
}
