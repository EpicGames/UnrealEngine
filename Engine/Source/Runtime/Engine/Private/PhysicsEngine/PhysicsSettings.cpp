// Copyright Epic Games, Inc. All Rights Reserved.

#include "PhysicsEngine/PhysicsSettings.h"
#include "GameFramework/MovementComponent.h"
#include "PhysicalMaterials/PhysicalMaterial.h"
#include "PhysicsCoreTypes.h"
#include "PhysicsEngine/BodySetup.h"
#include "UObject/UObjectIterator.h"



#include UE_INLINE_GENERATED_CPP_BY_NAME(PhysicsSettings)

UPhysicsSettings::UPhysicsSettings(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, LockedAxis_DEPRECATED(ESettingsLockedAxis::Invalid)
	, bSuppressFaceRemapTable(false)
	, bDisableActiveActors(false)
	, AnimPhysicsMinDeltaTime(0.f)
	, bSimulateAnimPhysicsAfterReset(false)
	, MinPhysicsDeltaTime(UE_SMALL_NUMBER)
	, MaxPhysicsDeltaTime(1.f / 30.f)
	, bSubstepping(false)
	, bTickPhysicsAsync(false)
	, AsyncFixedTimeStepSize(1.f / 30.f)
	, MaxSubstepDeltaTime(1.f / 60.f)
	, MaxSubsteps(6)
	, SyncSceneSmoothingFactor(0.0f)
	, InitialAverageFrameRate(1.f / 60.f)
	, PhysXTreeRebuildRate(10)
	, MinDeltaVelocityForHitEvents(0.f)
	, bPartialIslandSleep(false)
	, MomentumPropagation(0.3f)
	, PostStepWakeThreshold(20.0f)
	, MinimumMotionless(0.1f)
	, LinearWakeThresholdMultiplier(10.0f)
	, AngularWakeThresholdMultiplier(10.0f)
{
}

void UPhysicsSettings::PostInitProperties()
{
	Super::PostInitProperties();
#if WITH_EDITOR
	LoadSurfaceType();

	if (IsTemplate())
	{
		ImportConsoleVariableValues();
	}
#endif // #if WITH_EDITOR

	if (PhysicsPrediction.bEnablePhysicsResimulation_DEPRECATED)
	{
		PhysicsPrediction.bEnablePhysicsHistoryCapture = true;
	}

	if (PhysicsPrediction.ResimulationErrorThreshold_DEPRECATED > 0.0f)
	{
		PhysicsPrediction.ResimulationSettings.bEnableResimulationErrorPositionThreshold = true;
		PhysicsPrediction.ResimulationSettings.ResimulationErrorPositionThreshold = PhysicsPrediction.ResimulationErrorThreshold_DEPRECATED;
	}

	if (LockedAxis_DEPRECATED == static_cast<ESettingsLockedAxis::Type>(-1))
	{
		LockedAxis_DEPRECATED = ESettingsLockedAxis::Invalid;
	}

	if (LockedAxis_DEPRECATED != ESettingsLockedAxis::Invalid)
	{
		if (LockedAxis_DEPRECATED == ESettingsLockedAxis::None)
		{
			DefaultDegreesOfFreedom = ESettingsDOF::Full3D;
		}
		else if (LockedAxis_DEPRECATED == ESettingsLockedAxis::X)
		{
			DefaultDegreesOfFreedom = ESettingsDOF::YZPlane;
		}
		else if (LockedAxis_DEPRECATED == ESettingsLockedAxis::Y)
		{
			DefaultDegreesOfFreedom = ESettingsDOF::XZPlane;
		}
		else if (LockedAxis_DEPRECATED == ESettingsLockedAxis::Z)
		{
			DefaultDegreesOfFreedom = ESettingsDOF::XYPlane;
		}

		LockedAxis_DEPRECATED = ESettingsLockedAxis::Invalid;
	}

	// Temporarily override dedicated thread to taskgraph. The enum selection for dedicated
	// thread is hidden until that threading mode is made to work with the world physics system overall
	if(ChaosSettings.DefaultThreadingModel == EChaosThreadingMode::DedicatedThread)
	{
		ChaosSettings.DefaultThreadingModel = EChaosThreadingMode::TaskGraph;
	}

	// Override the core Chaos default settings with this one if its the CDO (the one edited in Project Settings)
	if (UPhysicsSettings::Get() == this)
	{
		UPhysicsSettingsCore::SetDefaultSettings(this);
	}
}

#if WITH_EDITOR
bool UPhysicsSettings::CanEditChange(const FProperty* Property) const
{
	bool bIsEditable = Super::CanEditChange(Property);
	if(bIsEditable && Property != NULL)
	{
		const FName Name = Property->GetFName();
		if(Name == TEXT("MaxPhysicsDeltaTime") || Name == TEXT("SyncSceneSmoothingFactor") || Name == TEXT("AsyncSceneSmoothingFactor") || Name == TEXT("InitialAverageFrameRate"))
		{
			bIsEditable = !bSubstepping;
		}
	}

	return bIsEditable;
}

void UPhysicsSettings::PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	if (PropertyChangedEvent.Property)
	{
		ExportValuesToConsoleVariables(PropertyChangedEvent.Property);
	}

	const FName PropertyName = PropertyChangedEvent.Property ? PropertyChangedEvent.Property->GetFName() : NAME_None;
	if (PropertyName == GET_MEMBER_NAME_CHECKED(UPhysicsSettings, FrictionCombineMode) || PropertyName == GET_MEMBER_NAME_CHECKED(UPhysicsSettings, RestitutionCombineMode))
	{
		UPhysicalMaterial::RebuildPhysicalMaterials();
	}
	else if (PropertyName == GET_MEMBER_NAME_CHECKED(UPhysicsSettings, DefaultDegreesOfFreedom))
	{
		UMovementComponent::PhysicsLockedAxisSettingChanged();
	}

	const FName MemberName = PropertyChangedEvent.MemberProperty ? PropertyChangedEvent.MemberProperty->GetFName() : NAME_None;
	if(MemberName == GET_MEMBER_NAME_CHECKED(UPhysicsSettings, ChaosSettings))
	{
		ChaosSettings.OnSettingsUpdated();
	}

	if(PropertyChangedEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(UPhysicsSettingsCore, DefaultShapeComplexity))
	{
		for(TObjectIterator<UBodySetup> It; It; ++It)
		{
			UBodySetup* Setup = *It;
			check(Setup);

			if(Setup->bCreatedPhysicsMeshes)
			{
				Setup->InvalidatePhysicsData();
			}
		}
	}
}

void UPhysicsSettings::LoadSurfaceType()
{
	// read "SurfaceType" defines and set meta data for the enum
	// find the enum
	UEnum * Enum = StaticEnum<EPhysicalSurface>();
	// we need this Enum
	check(Enum);

	const FString KeyName = "DisplayName";
	const FString HiddenMeta = "Hidden";
	const FString UnusedDisplayName = "Unused";

	// remainders, set to be unused
	for(int32 EnumIndex=1; EnumIndex<Enum->NumEnums(); ++EnumIndex)
	{
		// if meta data isn't set yet, set name to "Unused" until we fix property window to handle this
		// make sure all hide and set unused first
		// if not hidden yet
		if(!Enum->HasMetaData(*HiddenMeta, EnumIndex))
		{
			Enum->SetMetaData(*HiddenMeta, TEXT(""), EnumIndex);
			Enum->SetMetaData(*KeyName, *UnusedDisplayName, EnumIndex);
		}
	}

	for(auto Iter=PhysicalSurfaces.CreateConstIterator(); Iter; ++Iter)
	{
		// @todo only for editor
		Enum->SetMetaData(*KeyName, *Iter->Name.ToString(), Iter->Type);
		// also need to remove "Hidden"
		Enum->RemoveMetaData(*HiddenMeta, Iter->Type);
	}
}

#endif	// WITH_EDITOR

FChaosPhysicsSettings::FChaosPhysicsSettings() 
	: DefaultThreadingModel(EChaosThreadingMode::TaskGraph)
	, DedicatedThreadTickMode(EChaosSolverTickMode::VariableCappedWithTarget)
	, DedicatedThreadBufferMode(EChaosBufferMode::Double)
{

}

void FChaosPhysicsSettings::OnSettingsUpdated()
{

}

