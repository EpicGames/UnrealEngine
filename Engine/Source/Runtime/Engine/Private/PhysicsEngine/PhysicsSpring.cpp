// Copyright Epic Games, Inc. All Rights Reserved.

#include "Engine/World.h"
#include "Components/PrimitiveComponent.h"
#include "PhysicsEngine/PhysicsSpringComponent.h"


UPhysicsSpringComponent::UPhysicsSpringComponent(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	PrimaryComponentTick.bCanEverTick = true;
	PrimaryComponentTick.TickGroup = TG_PostPhysics;
	bAutoActivate = true;

	SpringStiffness = 25.f;
	SpringDamping = 3.f;
	SpringLengthAtRest = 100.f;
	SpringRadius = 10.f;
	SpringChannel = ECC_WorldDynamic;
	bIgnoreSelf = true;

	SpringCompression = 0;
}
FVector UPhysicsSpringComponent::SpringPositionFromLength(float Length) const
{
	return GetComponentTransform().GetLocation() + Length * GetSpringDirection();
}

FVector UPhysicsSpringComponent::GetSpringDirection() const
{
	return GetComponentTransform().TransformVectorNoScale(FVector(1.f, 0.f, 0.f));
}

float UPhysicsSpringComponent::GetNormalizedCompressionScalar() const
{
	const float NormalizedComprssion = SpringCompression / SpringLengthAtRest;
	return FMath::Clamp(NormalizedComprssion, 0.f, 1.f);
}

FVector UPhysicsSpringComponent::GetSpringCurrentEndPoint() const
{
	return SpringCompression == 0 ? GetSpringRestingPoint() : CurrentEndPoint;
}

FVector UPhysicsSpringComponent::GetSpringRestingPoint() const
{
	return SpringPositionFromLength(SpringLengthAtRest);
}


UPrimitiveComponent* UPhysicsSpringComponent::GetSpringCollision(const FVector& Start, const FVector& End, float& Time) const
{
	UWorld* World = GetWorld();
	AActor* IgnoreActor = bIgnoreSelf ? GetOwner() : nullptr;

	FCollisionQueryParams QueryParams(SCENE_QUERY_STAT(SpringComponent), true, IgnoreActor);
	FHitResult Hit;

	UPrimitiveComponent* CollidedComponent = nullptr;
	
	const FVector Delta = End - Start;
	const float DeltaSizeSqr = Delta.SizeSquared();
	if (DeltaSizeSqr > FMath::Square(UE_SMALL_NUMBER))
	{
		if (bool bBlockingHit = World->SweepSingleByChannel(Hit, Start, End, FQuat::Identity, SpringChannel, FCollisionShape::MakeSphere(SpringRadius), QueryParams))
		{
			CollidedComponent = Hit.GetComponent();
			Time = CollidedComponent ? Hit.Time : 1.f;
		}
	}

	return CollidedComponent;
}

FVector UPhysicsSpringComponent::ComputeNewSpringCompressionAndForce(const FVector& End, const float DeltaTime, float& NewSpringCompression) const
{
	const FVector SpringDir = GetSpringDirection();
	const FVector RestPoint = SpringPositionFromLength(SpringLengthAtRest);
	NewSpringCompression = (End - RestPoint).Size();

	const float SpringCompressionDelta = (NewSpringCompression - SpringCompression) / DeltaTime;
	return SpringDir * ((-SpringStiffness*NewSpringCompression) + (-SpringDamping*SpringCompressionDelta));
}

void UPhysicsSpringComponent::UpdateAttachedPosition() const
{
	const FVector EndPosition = GetSpringCurrentEndPoint();
	for (USceneComponent* ChildComponent : GetAttachChildren())
	{
		ChildComponent->SetWorldLocation(EndPosition);
	}
}

void UPhysicsSpringComponent::TickComponent(float DeltaTime, enum ELevelTick TickType, FActorComponentTickFunction *ThisTickFunction)
{
	Super::TickComponent(DeltaTime, TickType, ThisTickFunction);

	if (UPrimitiveComponent* BasePrimComp = Cast<UPrimitiveComponent>(GetAttachParent()))
	{
		if (IsActive())
		{
			const FVector SpringStart = GetComponentTransform().GetLocation();
			const FVector SpringDesiredEnd = SpringPositionFromLength(SpringLengthAtRest);
			float CollisionTime = 1.f;
			
			UPrimitiveComponent* CollideWith = GetSpringCollision(SpringStart, SpringDesiredEnd, CollisionTime);
			CurrentEndPoint = FMath::Lerp(SpringStart, SpringDesiredEnd, CollisionTime);
			
			if(CollideWith)
			{
				const float Mass = BasePrimComp->GetMass();
				float NewSpringCompression;
				const FVector WorldForce = ComputeNewSpringCompressionAndForce(CurrentEndPoint, DeltaTime, NewSpringCompression);
				
				BasePrimComp->AddForceAtLocation(WorldForce*Mass, SpringStart, GetAttachSocketName());
				SpringCompression = NewSpringCompression;
			}

			UpdateAttachedPosition();
		}
	}
}
