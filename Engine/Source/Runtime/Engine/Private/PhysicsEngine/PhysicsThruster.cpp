// Copyright Epic Games, Inc. All Rights Reserved.

#include "PhysicsEngine/PhysicsThruster.h"
#include "Async/TaskGraphInterfaces.h"
#include "Components/ArrowComponent.h"
#include "Components/BillboardComponent.h"
#include "Engine/Texture2D.h"
#include "PhysicsEngine/PhysicsThrusterComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PhysicsThruster)

#define LOCTEXT_NAMESPACE "PhysicsThrusterComponent"


UPhysicsThrusterComponent::UPhysicsThrusterComponent(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	PrimaryComponentTick.bCanEverTick = true;
	PrimaryComponentTick.TickGroup = TG_PrePhysics;
	ThrustStrength = 100.0;
#if WITH_EDITORONLY_DATA
	bVisualizeComponent = true;
#endif
}

#if WITH_EDITORONLY_DATA
void UPhysicsThrusterComponent::OnRegister()
{
	Super::OnRegister();

	if (SpriteComponent != nullptr)
	{
		SpriteComponent->SetSprite(LoadObject<UTexture2D>(NULL, TEXT("/Engine/EditorResources/S_Thruster.S_Thruster")));
		SpriteComponent->SetRelativeScale3D(FVector(0.5f));
		SpriteComponent->SpriteInfo.Category = TEXT("Physics");
		SpriteComponent->SpriteInfo.DisplayName = NSLOCTEXT("SpriteCategory", "Physics", "Physics");
		SpriteComponent->bIsScreenSizeScaled = true;
	}
}
#endif // WITH_EDITORONLY_DATA

void UPhysicsThrusterComponent::TickComponent(float DeltaTime, enum ELevelTick TickType, FActorComponentTickFunction *ThisTickFunction)
{
	Super::TickComponent(DeltaTime, TickType, ThisTickFunction);

	// Applied force to the base, so if we don't have one, do nothing.
	if(IsActive() && GetAttachParent())
	{
		FVector WorldForce = ThrustStrength * GetComponentTransform().TransformVectorNoScale( FVector(-1.f,0.f,0.f) );

		UPrimitiveComponent* BasePrimComp = Cast<UPrimitiveComponent>(GetAttachParent());
		if(BasePrimComp)
		{
			BasePrimComp->AddForceAtLocation(WorldForce, GetComponentLocation(), NAME_None);
		}
	}
}

//////////////////////////////////////////////////////////////////////////

APhysicsThruster::APhysicsThruster(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	ThrusterComponent = CreateDefaultSubobject<UPhysicsThrusterComponent>(TEXT("Thruster0"));
	RootComponent = ThrusterComponent;

#if WITH_EDITORONLY_DATA
	ArrowComponent = CreateEditorOnlyDefaultSubobject<UArrowComponent>(TEXT("ArrowComponent0"));

	if (!IsRunningCommandlet())
	{
		if (ArrowComponent)
		{
			ArrowComponent->ArrowSize = 1.7f;
			ArrowComponent->ArrowColor = FColor(255, 180, 0);

			ArrowComponent->bTreatAsASprite = true;
			ArrowComponent->SpriteInfo.Category = TEXT("Physics");
			ArrowComponent->SpriteInfo.DisplayName = NSLOCTEXT("SpriteCategory", "Physics", "Physics");
			ArrowComponent->SetupAttachment(ThrusterComponent);
			ArrowComponent->bIsScreenSizeScaled = true;
		}
	}
#endif // WITH_EDITORONLY_DATA
}

#undef LOCTEXT_NAMESPACE

