// Copyright Epic Games, Inc. All Rights Reserved.

#include "PhysicsEngine/RadialForceComponent.h"
#include "PhysicsEngine/RigidBodyBase.h"
#include "UObject/ConstructorHelpers.h"
#include "Engine/OverlapResult.h"
#include "Engine/World.h"
#include "Components/BillboardComponent.h"
#include "Engine/Texture2D.h"
#include "GameFramework/MovementComponent.h"
#include "PhysicsEngine/RadialForceActor.h"
#include "DestructibleInterface.h"
#include "UObject/UnrealType.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RadialForceComponent)

//////////////////////////////////////////////////////////////////////////
// RADIALFORCECOMPONENT
URadialForceComponent::URadialForceComponent(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	PrimaryComponentTick.bCanEverTick = true;
	PrimaryComponentTick.TickGroup = TG_PrePhysics;
	Radius = 200.0f;
	Falloff = RIF_Constant;
	ImpulseStrength = 1000.0f;
	ForceStrength = 10.0f;
	bAutoActivate = true;

	// by default we affect all 'dynamic' objects that can currently be affected by forces
	AddCollisionChannelToAffect(ECC_Pawn);
	AddCollisionChannelToAffect(ECC_PhysicsBody);
	AddCollisionChannelToAffect(ECC_Vehicle);
	AddCollisionChannelToAffect(ECC_Destructible);

	UpdateCollisionObjectQueryParams();
}

void URadialForceComponent::TickComponent(float DeltaTime, enum ELevelTick TickType, FActorComponentTickFunction *ThisTickFunction)
{
	Super::TickComponent(DeltaTime, TickType, ThisTickFunction);

	if(IsActive())
	{
		const FVector Origin = GetComponentLocation();

		// Find objects within the sphere
		TArray<FOverlapResult> Overlaps;

		FCollisionQueryParams Params(SCENE_QUERY_STAT(AddForceOverlap), false);

		// Ignore owner actor if desired
		if (bIgnoreOwningActor)
		{
			Params.AddIgnoredActor(GetOwner());
		}

		GetWorld()->OverlapMultiByObjectType(Overlaps, Origin, FQuat::Identity, CollisionObjectQueryParams, FCollisionShape::MakeSphere(Radius), Params);

		// A component can have multiple physics presences (e.g. destructible mesh components).
		// The component should handle the radial force for all of the physics objects it contains
		// so here we grab all of the unique components to avoid applying impulses more than once.
		TArray<UPrimitiveComponent*, TInlineAllocator<1>> AffectedComponents;
		AffectedComponents.Reserve(Overlaps.Num());

		for(FOverlapResult& OverlapResult : Overlaps)
		{
			if(UPrimitiveComponent* PrimitiveComponent = OverlapResult.Component.Get())
			{
				AffectedComponents.AddUnique(PrimitiveComponent);
			}
		}

		for(UPrimitiveComponent* PrimitiveComponent : AffectedComponents)
		{
			PrimitiveComponent->AddRadialForce(Origin, Radius, ForceStrength, Falloff);

			// see if this is a target for a movement component
			AActor* ComponentOwner = PrimitiveComponent->GetOwner();
			if(ComponentOwner)
			{
				TInlineComponentArray<UMovementComponent*> MovementComponents;
				ComponentOwner->GetComponents(MovementComponents);
				for(const auto& MovementComponent : MovementComponents)
				{
					if(MovementComponent->UpdatedComponent == PrimitiveComponent)
					{
						MovementComponent->AddRadialForce(Origin, Radius, ForceStrength, Falloff);
						break;
					}
				}
			}
		}
	}
}

void URadialForceComponent::BeginPlay()
{
	Super::BeginPlay();

	UpdateCollisionObjectQueryParams();
}

void URadialForceComponent::PostLoad()
{
	Super::PostLoad();

	UpdateCollisionObjectQueryParams();
}

void URadialForceComponent::FireImpulse()
{
	const FVector Origin = GetComponentLocation();

	// Find objects within the sphere
	TArray<FOverlapResult> Overlaps;

	FCollisionQueryParams Params(SCENE_QUERY_STAT(FireImpulseOverlap),  false);

	// Ignore owner actor if desired
	if (bIgnoreOwningActor)
	{
		Params.AddIgnoredActor(GetOwner());
	}

	GetWorld()->OverlapMultiByObjectType(Overlaps, Origin, FQuat::Identity, CollisionObjectQueryParams, FCollisionShape::MakeSphere(Radius), Params);

	// A component can have multiple physics presences (e.g. destructible mesh components).
	// The component should handle the radial force for all of the physics objects it contains
	// so here we grab all of the unique components to avoid applying impulses more than once.
	TArray<UPrimitiveComponent*, TInlineAllocator<1>> AffectedComponents;
	AffectedComponents.Reserve(Overlaps.Num());

	for(FOverlapResult& OverlapResult : Overlaps)
	{
		if(UPrimitiveComponent* PrimitiveComponent = OverlapResult.Component.Get())
		{
			AffectedComponents.AddUnique(PrimitiveComponent);
		}
	}

	for(UPrimitiveComponent* PrimitiveComponent : AffectedComponents)
	{
		if(DestructibleDamage > UE_SMALL_NUMBER)
		{
			if(IDestructibleInterface* DestructibleInstance = Cast<IDestructibleInterface>(PrimitiveComponent))
			{
				DestructibleInstance->ApplyRadiusDamage(DestructibleDamage, Origin, Radius, ImpulseStrength, Falloff == RIF_Constant);
			}
		}

		// Apply impulse
		PrimitiveComponent->AddRadialImpulse(Origin, Radius, ImpulseStrength, Falloff, bImpulseVelChange);

		// See if this is a target for a movement component, if so apply the impulse to it
		if (PrimitiveComponent->bIgnoreRadialImpulse == false)
		{
			TInlineComponentArray<UMovementComponent*> MovementComponents;
			if(AActor* OwningActor = PrimitiveComponent->GetOwner())
			{
				OwningActor->GetComponents(MovementComponents);
				for(const auto& MovementComponent : MovementComponents)
				{
					if(MovementComponent->UpdatedComponent == PrimitiveComponent)
					{
						MovementComponent->AddRadialImpulse(Origin, Radius, ImpulseStrength, Falloff, bImpulseVelChange);
						break;
					}
				}
			}
		}
	}
}

void URadialForceComponent::AddCollisionChannelToAffect(enum ECollisionChannel CollisionChannel)
{
	EObjectTypeQuery ObjectType = UEngineTypes::ConvertToObjectType(CollisionChannel);
	if(ObjectType != ObjectTypeQuery_MAX)
	{
		AddObjectTypeToAffect(ObjectType);
	}
}

void URadialForceComponent::AddObjectTypeToAffect(TEnumAsByte<enum EObjectTypeQuery> ObjectType)
{
	ObjectTypesToAffect.AddUnique(ObjectType);
	UpdateCollisionObjectQueryParams();
}

void URadialForceComponent::RemoveObjectTypeToAffect(TEnumAsByte<enum EObjectTypeQuery> ObjectType)
{
	ObjectTypesToAffect.Remove(ObjectType);
	UpdateCollisionObjectQueryParams();
}

#if WITH_EDITOR

void URadialForceComponent::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	// If we have edited the object types to effect, update our bitfield.
	if(PropertyChangedEvent.Property && PropertyChangedEvent.Property->GetFName() == TEXT("ObjectTypesToAffect"))
	{
		UpdateCollisionObjectQueryParams();
	}

	Super::PostEditChangeProperty(PropertyChangedEvent);
}

#endif

void URadialForceComponent::UpdateCollisionObjectQueryParams()
{
	CollisionObjectQueryParams = FCollisionObjectQueryParams(ObjectTypesToAffect);
}


//////////////////////////////////////////////////////////////////////////
// ARB_RADIALFORCEACTOR
ARadialForceActor::ARadialForceActor(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	ForceComponent = CreateDefaultSubobject<URadialForceComponent>(TEXT("ForceComponent0"));

#if WITH_EDITOR
	SpriteComponent = CreateEditorOnlyDefaultSubobject<UBillboardComponent>(TEXT("Sprite"));
	if (SpriteComponent)
	{
		// Structure to hold one-time initialization
		if (!IsRunningCommandlet())
		{
			struct FConstructorStatics
			{
				ConstructorHelpers::FObjectFinderOptional<UTexture2D> RadialForceTexture;
				FName ID_Physics;
				FText NAME_Physics;
				FConstructorStatics()
					: RadialForceTexture(TEXT("/Engine/EditorResources/S_RadForce.S_RadForce"))
					, ID_Physics(TEXT("Physics"))
					, NAME_Physics(NSLOCTEXT( "SpriteCategory", "Physics", "Physics" ))
				{
				}
			};
			static FConstructorStatics ConstructorStatics;

			SpriteComponent->Sprite = ConstructorStatics.RadialForceTexture.Get();

#if WITH_EDITORONLY_DATA
			SpriteComponent->SpriteInfo.Category = ConstructorStatics.ID_Physics;
			SpriteComponent->SpriteInfo.DisplayName = ConstructorStatics.NAME_Physics;
#endif // WITH_EDITORONLY_DATA
		}

		SpriteComponent->SetRelativeScale3D(FVector(0.5f, 0.5f, 0.5f));
		SpriteComponent->SetupAttachment(ForceComponent);
		SpriteComponent->bIsScreenSizeScaled = true;
	}
#endif

	RootComponent = ForceComponent;
	SetRemoteRoleForBackwardsCompat(ROLE_SimulatedProxy);
	bReplicates = true;
	bAlwaysRelevant = true;
	SetNetUpdateFrequency(0.1f);
}

#if WITH_EDITOR
void ARadialForceActor::EditorApplyScale(const FVector& DeltaScale, const FVector* PivotLocation, bool bAltDown, bool bShiftDown, bool bCtrlDown)
{
	FVector ModifiedScale = DeltaScale * ( AActor::bUsePercentageBasedScaling ? 500.0f : 5.0f );

	const float Multiplier = ( ModifiedScale.X > 0.0f || ModifiedScale.Y > 0.0f || ModifiedScale.Z > 0.0f ) ? 1.0f : -1.0f;
	if(ForceComponent)
	{
		ForceComponent->Radius += Multiplier * ModifiedScale.Size();
		ForceComponent->Radius = FMath::Max( 0.f, ForceComponent->Radius );
	}
}
#endif

void ARadialForceActor::FireImpulse()
{
	if(ForceComponent)
	{
		ForceComponent->FireImpulse();
	}
}

void ARadialForceActor::EnableForce()
{
	if(ForceComponent)
	{
		ForceComponent->Activate();
	}
}

void ARadialForceActor::DisableForce()
{
	if(ForceComponent)
	{
		ForceComponent->Deactivate();
	}
}

void ARadialForceActor::ToggleForce()
{
	if(ForceComponent)
	{
		ForceComponent->ToggleActive();
	}
}
