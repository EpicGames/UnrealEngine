// Copyright Epic Games, Inc. All Rights Reserved.

#include "PhysicsEngine/ScopedSQHitchRepeater.h"

#if DETECT_SQ_HITCHES
int FSQHitchRepeaterCVars::SQHitchDetection = 0;
FAutoConsoleVariableRef FSQHitchRepeaterCVars::CVarSQHitchDetection(TEXT("p.SQHitchDetection"), FSQHitchRepeaterCVars::SQHitchDetection,
	TEXT("Whether to detect scene query hitches. 0 is off. 1 repeats a slow scene query once and prints extra information. 2+ repeat slow query n times without recording (useful when profiling)")
);

int FSQHitchRepeaterCVars::SQHitchDetectionForceNames = 0;
FAutoConsoleVariableRef FSQHitchRepeaterCVars::CVarSQHitchDetectionForceNames(TEXT("p.SQHitchDetectionForceNames"), FSQHitchRepeaterCVars::SQHitchDetectionForceNames,
	TEXT("Whether name resolution is forced off the game thread. This is not 100% safe, but can be useful when looking at hitches off GT")
);

float FSQHitchRepeaterCVars::SQHitchDetectionThreshold = 0.05f;
FAutoConsoleVariableRef FSQHitchRepeaterCVars::CVarSQHitchDetectionThreshold(TEXT("p.SQHitchDetectionThreshold"), FSQHitchRepeaterCVars::SQHitchDetectionThreshold,
	TEXT("Determines the threshold in milliseconds for a scene query hitch.")
);
#endif