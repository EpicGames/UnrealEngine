// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Engine/EngineTypes.h"
#include "ProfilingDebugging/ScopedTimers.h"
#include "HAL/IConsoleManager.h"
#include "PhysXPublic.h"
#include "CollisionQueryParams.h"
#include "CollisionQueryFilterCallback.h"
#include "WorldCollision.h"

#if DETECT_SQ_HITCHES
struct FSQHitchRepeaterCVars
{
	static int SQHitchDetection;
	static FAutoConsoleVariableRef CVarSQHitchDetection;

	static int SQHitchDetectionForceNames;
	static FAutoConsoleVariableRef CVarSQHitchDetectionForceNames;

	static float SQHitchDetectionThreshold;
	static FAutoConsoleVariableRef CVarSQHitchDetectionThreshold;
};
#endif 

/** Various info we want to capture for hitch detection reporting */
struct FHitchDetectionInfo
{
#if DETECT_SQ_HITCHES
	FVector Start;
	FVector End;
	FTransform Pose;
	ECollisionChannel TraceChannel;
	const FCollisionQueryParams& Params;
	bool bInTM;

	FHitchDetectionInfo(const FVector& InStart, const FVector& InEnd, ECollisionChannel InTraceChannel, const FCollisionQueryParams& InParams)
		: Start(InStart)
		, End(InEnd)
		, TraceChannel(InTraceChannel)
		, Params(InParams)
		, bInTM(false)
	{
	}

	FHitchDetectionInfo(const FTransform& InPose, ECollisionChannel InTraceChannel, const FCollisionQueryParams& InParams)
		: Pose(InPose)
		, TraceChannel(InTraceChannel)
		, Params(InParams)
		, bInTM(true)
	{
	}

	FString ToString() const
	{
		if (bInTM)
		{
			return FString::Printf(TEXT("Pose:%s TraceChannel:%d Params:%s"), *Pose.ToString(), (int32)TraceChannel, *Params.ToString());
		}
		else
		{
			return FString::Printf(TEXT("Start:%s End:%s TraceChannel:%d Params:%s"), *Start.ToString(), *End.ToString(), (int32)TraceChannel, *Params.ToString());
		}
	};
#else
	FHitchDetectionInfo(const FVector&, const FVector&, ECollisionChannel, const FCollisionQueryParams&) {}
	FHitchDetectionInfo(const FTransform& InPose, ECollisionChannel InTraceChannel, const FCollisionQueryParams& InParams) {}
	FString ToString() const { return FString(); }
#endif // DETECT_SQ_HITCHES
};

template <typename BufferType>
struct FScopedSQHitchRepeater
{
#if DETECT_SQ_HITCHES
	double HitchDuration;
	FDurationTimer HitchTimer;
	int32 LoopCounter;
	BufferType& UserBuffer;	//The buffer the user would normally use when no repeating happens
	BufferType* OriginalBuffer;	//The buffer as it was before the query, this is needed to maintain the same buffer properties for each loop
	BufferType* RepeatBuffer;			//Dummy buffer for loops
	FCollisionQueryFilterCallback& QueryCallback;
	FHitchDetectionInfo HitchDetectionInfo;

	bool RepeatOnHitch()
	{
		if (FSQHitchRepeaterCVars::SQHitchDetection)
		{
			if (LoopCounter == 0)
			{
				HitchTimer.Stop();
			}

			const bool bLoop = (LoopCounter < FSQHitchRepeaterCVars::SQHitchDetection) && (HitchDuration * 1000.0) >= FSQHitchRepeaterCVars::SQHitchDetectionThreshold;
			++LoopCounter;

			QueryCallback.bRecordHitches = QueryCallback.bRecordHitches ? true : bLoop && FSQHitchRepeaterCVars::SQHitchDetection == 1;
			if (bLoop)
			{
				if (!RepeatBuffer)
				{
					RepeatBuffer = new BufferType(*OriginalBuffer);
				}
				else
				{
					*RepeatBuffer = *OriginalBuffer;	//make a copy to make sure we have the same behavior every iteration
				}
			}
			return bLoop;
		}
		else
		{
			return false;
		}
	}

	FScopedSQHitchRepeater(BufferType& OutBuffer, FCollisionQueryFilterCallback& QueryCallback, const FHitchDetectionInfo& InHitchDetectionInfo)
		: HitchDuration(0.0)
		, HitchTimer(HitchDuration)
		, LoopCounter(0)
		, UserBuffer(OutBuffer)
		, OriginalBuffer(nullptr)
		, RepeatBuffer(nullptr)
		, QueryCallback(QueryCallback)
		, HitchDetectionInfo(InHitchDetectionInfo)
	{
		if (FSQHitchRepeaterCVars::SQHitchDetection)
		{
			OriginalBuffer = new BufferType(UserBuffer);
			HitchTimer.Start();
		}
	}

	BufferType& GetBuffer()
	{
		return LoopCounter == 0 ? UserBuffer : *RepeatBuffer;
	}

	~FScopedSQHitchRepeater()
	{
		if (QueryCallback.bRecordHitches)
		{
			const double DurationInMS = HitchDuration * 1000.0;
			UE_LOGF(LogCollision, Warning, "SceneQueryHitch: took %.3fms with %d calls to PreFilter", DurationInMS, QueryCallback.PreFilterHitchInfo.Num());
			UE_LOGF(LogCollision, Warning, "\t%ls", *HitchDetectionInfo.ToString());
			for (const FCollisionQueryFilterCallback::FPreFilterRecord& Record : QueryCallback.PreFilterHitchInfo)
			{
				UE_LOGF(LogCollision, Warning, "\tPreFilter:%ls, result=%d", *Record.OwnerComponentReadableName, (int32)Record.Result);
			}
			QueryCallback.PreFilterHitchInfo.Empty();
		}

		QueryCallback.bRecordHitches = false;
		delete RepeatBuffer;
		delete OriginalBuffer;
	}

#else
	FScopedSQHitchRepeater(BufferType& OutBuffer, FCollisionQueryFilterCallback& PQueryCallback, const FHitchDetectionInfo& InHitchDetectionInfo)
		: UserBuffer(OutBuffer)
	{
	}

	BufferType& UserBuffer;	//The buffer the user would normally use when no repeating happens

	BufferType& GetBuffer() const { return UserBuffer; }
	bool RepeatOnHitch() const { return false; }
#endif
};
