// Copyright Epic Games, Inc. All Rights Reserved.

#include "Physics/SimpleSuspension.h"
#include "Chaos/Real.h"


// Some calculations are expected to exceed the engine's SMALL_NUMBER threshold
static double SUSPENSION_SMALL_NUMBER = 1.e-10;

// Tolerance for using single axis calculations
static double SUSPENSION_ALIGNMENT_TOLERANCE = 0.1;

void FSimpleSuspension::Setup(const FSimpleSuspensionParams& InSuspensionParams)
{
	const int32 Count = SuspensionParams.SpringParams.Num();
	ensure(SuspensionParams.LocalSpringOrigins.Num() == Count);
	SuspensionState.SpringDisplacements.SetNumZeroed(Count);
	SuspensionState.SpringForces.SetNumZeroed(Count);
	SuspensionState.SpringContacts.SetNumZeroed(Count);
	SuspensionState.WorldSpringOrigins.SetNumZeroed(Count);
	SuspensionParams = InSuspensionParams;
}

void FSimpleSuspension::Setup(const FSimpleSuspensionParams& InSuspensionParams, const float TotalMass, const float Gravity)
{
	Setup(InSuspensionParams);
	ComputeSpringParams(TotalMass, Gravity);
}

void FSimpleSuspension::ComputeSpringParams(const float TotalMass, const float Gravity)
{
	FSimpleSuspensionHelpers::ComputeSpringParams(SuspensionParams, TotalMass, Gravity, SuspensionParams.SpringParams);
}

void FSimpleSuspension::Update(const FTransform& LocalToWorld, const FVector& LinearVelocity, const FVector& AngularVelocityRad, const FSimpleSuspensionRaycast& RaycastFunction)
{
	// Compute world coordinates needed for displacement and force calculations
	FSimpleSuspensionHelpers::ComputeWorldSuspensionCoordinates(SuspensionParams, LocalToWorld, SuspensionState);

	// Do raycasts to find the displacements of each spring
	FSimpleSuspensionHelpers::ComputeSuspensionDisplacements(SuspensionParams, RaycastFunction, SuspensionState);

	// Compute the forces which result from spring displacements
	FSimpleSuspensionHelpers::ComputeSuspensionForces(LinearVelocity, AngularVelocityRad, SuspensionState, SuspensionParams.SpringParams, SuspensionState);
}

bool FSimpleSuspensionHelpers::ComputeSingleAxisLambda(const FVector::FReal AxisDot, const FVector::FReal SumAxis, const uint32 Count, TArray<FVector::FReal, TFixedAllocator<2>>& Lambdas, FString* ErrMsg)
{
	using Chaos::FReal;

	//compute determinant
	const FReal DetLL = AxisDot * Count - SumAxis * SumAxis;
	if (!ErrCheck(!FMath::IsNearlyZero(DetLL, SUSPENSION_SMALL_NUMBER),
		TEXT("Spring configuration is invalid! Please make sure no two springs are at the same location."), ErrMsg))
	{
		return false;
	}

	const FReal LambdaB0 = Lambdas[0];
	const FReal LambdaB1 = Lambdas[1];

	//Compute the inverse matrix
	const FReal DetLLInv = 1.f / DetLL;
	const FReal InvLL00 = Count * DetLLInv;
	const FReal InvLL01 = -SumAxis * DetLLInv;
	const FReal InvLL10 = -SumAxis * DetLLInv;
	const FReal InvLL11 = AxisDot * DetLLInv;

	//compute Lagrange Multipliers - The third lambda value will always be zero in the 1D case.
	Lambdas[0] = InvLL00 * LambdaB0 + InvLL01 * LambdaB1;
	Lambdas[1] = InvLL10 * LambdaB0 + InvLL11 * LambdaB1;

	return true;
}

bool FSimpleSuspensionHelpers::ComputeSprungMasses(const TArray<FVector>& MassSpringPositions, const float TotalMass, TArray<float>& OutSprungMasses, FString* ErrMsg)
{
	using Chaos::FReal;
	/*

	For a body which is supported by a collection of parallel springs,
	this method will compute a distribution of masses among the springs
	which minimizes the variance between them.

	This method assumes that spring positions are given relative to the
	center of mass of the body, and that gravity occurs in the local -Z
	direction.

	Different methods are used for 1, 2, and >= 3 numbers of springs.

	*/

	// Make sure we have enough space in the spring mass results array
	const uint32 Count = MassSpringPositions.Num();
	OutSprungMasses.Reserve(Count);

	// Check essential values
	if (!ErrCheck(Count > 0, TEXT("Must have at least one spring to compute sprung masses."), ErrMsg))
	{
		return false;
	}

	if (!ErrCheck(TotalMass > UE_SMALL_NUMBER, TEXT("Total mass must be greater than zero to compute sprung masses."), ErrMsg))
	{
		return false;
	}

	// The cases of one or two springs are special snowflakes
	if (Count == 1)
	{
		OutSprungMasses[0] = TotalMass;
		return true;
	}
	else if (Count == 2)
	{
		/*

		For two springs, we project the CM (which is 0,0 since we're
		in the mass frame of the object) onto the line between the
		springs, and compute the ratio of the distances.

		For example in the graph below, if the springs are at points
		A and B, then the distances d0 and d1 will be computed.

			 d0      d1
		A---------p------B
				  |
				  c

		Then the masses m0 and m1 at A and B will be distributed according
		to the magnitudes of d0 and d1.

		m0 = m * d0 / (d0 + d1);
		m1 = m - m0;

		*/

		const FReal AX = MassSpringPositions[0].X;
		const FReal AY = MassSpringPositions[0].Y;
		const FReal DiffX = MassSpringPositions[1].X - AX;
		const FReal DiffY = MassSpringPositions[1].Y - AY;

		// If the springs are close together, just divide the mass in 2.
		const FReal DistSquared = (DiffX * DiffX) + (DiffY * DiffY);
		const FReal Dist = FMath::Sqrt(DistSquared);
		if (Dist <= UE_SMALL_NUMBER)
		{
			OutSprungMasses[0] = OutSprungMasses[1] = TotalMass * .5f;
			return true;
		}

		// The springs are far enough apart, compute the distribution
		const FReal DistInv = 1.f / Dist;
		const FReal DirX = DiffX * DistInv;
		const FReal DirY = DiffY * DistInv;
		const FReal DirDotA = (DirX * AX) + (DirY * AY);
		OutSprungMasses[0] = -TotalMass * DirDotA * DistInv;
		OutSprungMasses[1] = TotalMass - OutSprungMasses[0];
		if (ErrCheck(OutSprungMasses[0] >= 0.f, TEXT("Spring configuration is invalid! Please make sure the center of mass is located between the springs."), ErrMsg) &&
			ErrCheck(OutSprungMasses[1] >= 0.f, TEXT("Spring configuration is invalid! Please make sure the center of mass is located between the springs."), ErrMsg))
		{
			return true;
		}
		return false;
	}

	/*

	In the case that we have N >= 3 springs, we solve a constrained minimization
	problem using Lagrange multipliers (https://en.wikipedia.org/wiki/Lagrange_multiplier).

	Our constraints express the following: the mass-weighted sum of the spring
	positions must equal the center of mass, and the sum of the sprung masses must equal
	the total mass of the body. Thus, we have three constraint equations:

	g0 = sum(x_i * m_i) = x_c * m
	g1 = sum(y_i * m_i) = y_c * m
	g2 = sum(m_i) = m

	When there are many springs, there are many ways in which we could distribute the
	masses among them. In order to get an even distribution, we want to minimize the
	function whose value is the total mass variance.

	f = sum((m_i - m_u)^2)

	where m_u is the average mass per spring, m / N.

	The equation which constrains the minimization of f will be the gradient of the sum
	of f and the constraints multiplied by scalars, set to zero.

	grad(f) + lambda0 grad(g0) + lambda1 grad(g1) + lambda2 grad(g2) = 0

	That is,

	2 m_i + lambda0 * x_i + lambda1 * y_i + lambda2 = 2m_u

	where 0 < i <= N. In combination with the constraint equations, we now have a system of
	N+3 equations and N+3 unknowns.

	[  0   0   0   x0  x1  x2  ... ] [ lambda0 ] = [ m x_c ]
	[  0   0   0   y0  y1  y2  ... ] [ lambda1 ]   [ m y_c ]
	[  0   0   0   1   1   1   ... ] [ lambda2 ]   [ m     ]
	[  x0  y0  1   2   0   0   ... ] [ m0      ]   [ 2 m_u ]
	[  x1  y1  1   0   2   0       ] [ m1      ]   [ 2 m_u ]
	[  x2  y2  1   0   0   2       ] [ m2      ]   [ 2 m_u ]
	[  .   .   .             .     ] [ .       ]   [ .     ]
	[  .   .   .               .   ] [ .       ]   [ .     ]
	[  .   .   .                 . ] [ .       ]   [ .     ]


	The linear (N+3)x(N+3) system which results takes a form which can be simplified
	using the Schur complement (https://en.wikipedia.org/wiki/Schur_complement) of the
	system matrix.

	[  0   Lt  ] [ lambda_vec ] = [ u ]
	[  L   2 I ] [ m_vec      ]   [ v ]

	where Lt = Transpose(L), lambda_vec = { lambda0, lambda1, lambda2 },
	m_vec = { m0, m1, m2, ... }, u = m { x_c, y_c, 1 }, and v = 2 m_u { 1, 1, 1, ...}.

	This system can be solved for lambda_vec and subsequently m_vec, which is our solution vector.

	lambda_vec = (Lt L)^-1 (Lt v - 2 u)
	m_vec = (v - L lambda_vec) / 2

	The matrix (Lt L) is a 3x3 matrix, and its inverse is the only one which must be found
	in order to finally solve for the masses.

	*/

	// Cache values we'll need later, and clear out the results array
	const FReal CountN = (FReal)Count;
	const FReal CountInverse = 1.f / CountN;
	const FReal AverageMass = TotalMass * CountInverse;
	FReal SumX = 0.f;
	FReal SumY = 0.f;
	FReal XDotX = 0.f;
	FReal YDotY = 0.f;
	FReal XDotY = 0.f;
	FReal B0 = 0.f;
	FReal B1 = 0.f;
	FReal B2 = 0.f;
	bool bAlignedX = true;
	bool bAlignedY = true;

	for (uint32 Index = 0; Index < Count; ++Index)
	{
		const FReal X = MassSpringPositions[Index].X;
		const FReal Y = MassSpringPositions[Index].Y;
		SumX += X;
		SumY += Y;
		XDotX += X * X;
		YDotY += Y * Y;
		XDotY += X * Y;

		bAlignedX &= FMath::IsNearlyEqual(MassSpringPositions[Index].X, MassSpringPositions[0].X, SUSPENSION_ALIGNMENT_TOLERANCE);
		bAlignedY &= FMath::IsNearlyEqual(MassSpringPositions[Index].Y, MassSpringPositions[0].Y, SUSPENSION_ALIGNMENT_TOLERANCE);
	}

	//calculate the lambdas - we approximate the center of mass as zero for each axis
	FReal Lambda0 = 0.0f;
	FReal Lambda1 = 0.0f;
	FReal Lambda2 = 0.0f;

	const FReal LambdaB0 = 2.f * AverageMass * SumX;
	const FReal LambdaB1 = 2.f * AverageMass * SumY;
	const FReal LambdaB2 = (2.f * AverageMass * CountN) - (2.f * TotalMass);

	//if one axis is aligned, we actually lose a constraint/equation and we need to adjust our calculation
	if (bAlignedY)
	{
		TArray<Chaos::FReal, TFixedAllocator<2>> Lambdas;
		Lambdas.Add(LambdaB0);
		Lambdas.Add(LambdaB2);

		ComputeSingleAxisLambda(XDotX, SumX, Count, Lambdas);

		Lambda0 = Lambdas[0];
		Lambda2 = Lambdas[1];
	}

	else if (bAlignedX)
	{
		TArray<Chaos::FReal, TFixedAllocator<2>> Lambdas;
		Lambdas.Add(LambdaB1);
		Lambdas.Add(LambdaB2);

		ComputeSingleAxisLambda(YDotY, SumY, Count, Lambdas);

		Lambda1 = Lambdas[0];
		Lambda2 = Lambdas[1];
	}

	//2D constraint calculation
	else
	{
		// Compute determinant of system matrix, in prep for inversion
		const FReal DetLL
			= (XDotX * YDotY * Count)
			+ (2.f * XDotY * SumX * SumY)
			- (YDotY * SumX * SumX)
			- (XDotX * SumY * SumY)
			- (XDotY * XDotY * Count);

		// Make sure the matrix is invertible!
		if (!ErrCheck(!FMath::IsNearlyZero(DetLL, SUSPENSION_SMALL_NUMBER),
			TEXT("Spring configuration is invalid! Please make sure no two springs are at the same location."), ErrMsg))
		{
			return false;
		}

		// Compute the elements of the inverse matrix
		const FReal DetLLInv = 1.f / DetLL;
		const FReal InvLL00 = ((Count * YDotY) - (SumY * SumY)) * DetLLInv;
		const FReal InvLL01 = ((SumX * SumY) - (Count * XDotY)) * DetLLInv;
		const FReal InvLL02 = ((SumY * XDotY) - (SumX * YDotY)) * DetLLInv;
		const FReal InvLL10 = InvLL01; // Symmetry!
		const FReal InvLL11 = ((Count * XDotX) - (SumX * SumX)) * DetLLInv;
		const FReal InvLL12 = ((SumX * XDotY) - (SumY * XDotX)) * DetLLInv; // = InvLL21. Symmetry!
		const FReal InvLL20 = InvLL02; // Symmetry!
		const FReal InvLL21 = InvLL12; // Symmetry!
		const FReal InvLL22 = ((XDotX * YDotY) - (XDotY * XDotY)) * DetLLInv;

		// Compute the Lagrange multipliers
		Lambda0 = (InvLL00 * LambdaB0) + (InvLL01 * LambdaB1) + (InvLL02 * LambdaB2);
		Lambda1 = (InvLL10 * LambdaB0) + (InvLL11 * LambdaB1) + (InvLL12 * LambdaB2);
		Lambda2 = (InvLL20 * LambdaB0) + (InvLL21 * LambdaB1) + (InvLL22 * LambdaB2);		
	}

	// Compute the masses
	for (uint32 Index = 0; Index < Count; ++Index)
	{
		const FReal X = MassSpringPositions[Index].X;
		const FReal Y = MassSpringPositions[Index].Y;
		const FReal LLambda = (X * Lambda0) + (Y * Lambda1) + Lambda2;
		OutSprungMasses[Index] = AverageMass - (0.5f * LLambda);
		if (!ErrCheck(OutSprungMasses[Index] >= 0.f, TEXT("Spring configuration is invalid! Please make sure the center of mass is located inside the area covered by the springs."), ErrMsg))
		{
			return false;
		}
	}

	return true;
}

bool FSimpleSuspensionHelpers::ComputeSprungMasses(const TArray<FVector>& LocalSpringPositions, const FVector& LocalCenterOfMass, const float TotalMass, TArray<float>& OutSprungMasses, FString* ErrMsg)
{
	// Compute support origin's in center of mass space
	const int32 SpringCount = LocalSpringPositions.Num();
	TArray<FVector> MassSpringPositions;
	MassSpringPositions.SetNum(SpringCount);
	for (int32 Index = 0; Index < SpringCount; ++Index)
	{
		MassSpringPositions[Index] = LocalSpringPositions[Index] - LocalCenterOfMass;
	}

	// Do the calculation
	return ComputeSprungMasses(MassSpringPositions, TotalMass, OutSprungMasses, ErrMsg);
}

void FSimpleSuspensionHelpers::ComputeSpringNaturalFrequencyAndDampingRatio(const float SprungMass, const float SpringStiffness, const float SpringDamping, float& OutNaturalFrequency, float& OutDampingRatio)
{
	const float NaturalFrequency = SprungMass > UE_SMALL_NUMBER ? SpringStiffness / SprungMass : 0.f;
	const float CriticalDamping = ComputeSpringCriticalDamping(SprungMass, OutNaturalFrequency);
	OutNaturalFrequency = NaturalFrequency;
	OutDampingRatio = CriticalDamping > UE_SMALL_NUMBER ? SpringDamping / CriticalDamping : 1.f;
}

float FSimpleSuspensionHelpers::ComputeSpringStiffness(const float SprungMass, const float NaturalFrequency)
{
	return SprungMass * NaturalFrequency * NaturalFrequency;
}

float FSimpleSuspensionHelpers::ComputeSpringDamping(const float SprungMass, const float NaturalFrequency, const float DampingRatio)
{
	return DampingRatio * ComputeSpringCriticalDamping(SprungMass, NaturalFrequency);
}

float FSimpleSuspensionHelpers::ComputeSpringCriticalDamping(const float SprungMass, const float NaturalFrequency)
{
	return 2.f * NaturalFrequency * SprungMass;
}

float FSimpleSuspensionHelpers::ComputeSpringRestLength(const float SprungMass, const float NaturalFrequency, const float SuspendedLength, const float Gravity)
{
	const float SpringStiffness = ComputeSpringStiffness(SprungMass, NaturalFrequency);
	if (SpringStiffness > UE_SMALL_NUMBER)
	{
		// "Suspended Displacement" is the amount by which a spring this stiff will
		// be compressed by the force of gravity when it's resting
		const float SuspendedDisplacement = SprungMass * Gravity / SpringStiffness;

		// Suspended length plus the suspended displacement will be the total
		// un-compressed spring length.
		return SuspendedLength + SuspendedDisplacement;
	}
	else
	{
		return 0.f;
	}
}

void FSimpleSuspensionHelpers::ComputeSpringParams(const TArray<FVector>& LocalSuspensionOrigins, const FVector& LocalCenterOfMass, const float TotalMass, const FVector& LocalSuspensionNormal, const float LocalGroundDistance, const float NaturalFrequency, const float DampingRatio, const float Gravity, TArray<FSimpleSuspensionSpringParams>& OutSpringParams)
{
	// Make sure our output array is the right size
	const int32 Count = LocalSuspensionOrigins.Num();
	OutSpringParams.SetNum(Count);

	// Compute sprung mass percents
	TArray<float> SprungMasses;
	SprungMasses.SetNum(Count);
	FSimpleSuspensionHelpers::ComputeSprungMasses(LocalSuspensionOrigins, LocalCenterOfMass, TotalMass, SprungMasses);

	// Set suspension parameters for each spring
	const FVector GroundPoint = -LocalSuspensionNormal * LocalGroundDistance;
	for (int32 Index = 0; Index < Count; ++Index)
	{
		// Compute spring rest length and suspended length
		const FVector OriginGroundDiff = LocalSuspensionOrigins[Index] - GroundPoint;
		const float SuspendedLength = FVector::DotProduct(LocalSuspensionNormal, OriginGroundDiff);
		const float SprungMass = SprungMasses[Index];

		// Store sprung mass, spring stiffness, and spring damping in this
		// support's computed parameters.
		FSimpleSuspensionSpringParams& Params = OutSpringParams[Index];
		Params.Stiffness = FSimpleSuspensionHelpers::ComputeSpringStiffness(SprungMasses[Index], NaturalFrequency);
		Params.Damping = FSimpleSuspensionHelpers::ComputeSpringDamping(SprungMass, NaturalFrequency, DampingRatio);
		Params.Length = FSimpleSuspensionHelpers::ComputeSpringRestLength(SprungMass, NaturalFrequency, SuspendedLength, Gravity);
		Params.Mass = SprungMass;
	}
}

void FSimpleSuspensionHelpers::ComputeSpringParams(const FSimpleSuspensionParams& SuspensionParams, const float TotalMass, const float Gravity, TArray<FSimpleSuspensionSpringParams>& OutSpringParams)
{
	FSimpleSuspensionHelpers::ComputeSpringParams(SuspensionParams.LocalSpringOrigins, SuspensionParams.LocalCenterOfMass, TotalMass, SuspensionParams.LocalSuspensionNormal, SuspensionParams.LocalGroundDistance, SuspensionParams.NaturalFrequency, SuspensionParams.DampingRatio, Gravity, OutSpringParams);
}

void FSimpleSuspensionHelpers::ComputeWorldSuspensionCoordinates(const FVector& LocalCenterOfMass, const FVector& LocalSuspensionNormal, const TArray<FVector>& LocalSuspensionOrigins, const FTransform& LocalToWorld, FVector& OutWorldCenterOfMass, FVector& OutWorldSuspensionNormal, TArray<FVector>& OutWorldSuspensionOrigins)
{
	const int32 Count = LocalSuspensionOrigins.Num();
	ensure(OutWorldSuspensionOrigins.Num() == Count);
	OutWorldCenterOfMass = LocalToWorld.TransformPosition(LocalCenterOfMass);
	OutWorldSuspensionNormal = LocalToWorld.TransformVector(LocalSuspensionNormal);
	for (int32 Index = 0; Index < Count; ++Index)
	{
		OutWorldSuspensionOrigins[Index] = LocalToWorld.TransformPosition(LocalSuspensionOrigins[Index]);
	}
}

void FSimpleSuspensionHelpers::ComputeWorldSuspensionCoordinates(const FSimpleSuspensionParams& SuspensionParams, const FTransform& LocalToWorld, FSimpleSuspensionState& OutSuspensionState)
{
	FSimpleSuspensionHelpers::ComputeWorldSuspensionCoordinates(SuspensionParams.LocalCenterOfMass, SuspensionParams.LocalSuspensionNormal, SuspensionParams.LocalSpringOrigins, LocalToWorld, OutSuspensionState.WorldCenterOfMass, OutSuspensionState.WorldSuspensionNormal, OutSuspensionState.WorldSpringOrigins);
}

void FSimpleSuspensionHelpers::ComputeSuspensionDisplacements(const TArray<FVector> WorldSuspensionOrigins, const TArray<FSimpleSuspensionSpringParams>& SpringParams, const FVector& WorldSuspensionNormal, const FSimpleSuspensionRaycast& RaycastFunction, TArray<float>& OutSpringDisplacements)
{
	const int32 Count = WorldSuspensionOrigins.Num();
	ensure(SpringParams.Num() == Count);
	ensure(OutSpringDisplacements.Num() == Count);
	for (int32 Index = 0; Index < Count; ++Index)
	{
		// Do a raycast to compute spring compression
		const float RestLength = SpringParams[Index].Length;
		const FVector WorldBegin = WorldSuspensionOrigins[Index];
		const FVector WorldEnd = WorldBegin - (RestLength * WorldSuspensionNormal);
		FVector HitPoint;
		FVector HitNormal;
		if (RaycastFunction(WorldBegin, WorldEnd, Index, HitPoint, HitNormal))
		{
			// We hit something, so compute it's displacement
			const float CompressedLength = FVector::DotProduct(WorldBegin - HitPoint, WorldSuspensionNormal);
			OutSpringDisplacements[Index] = RestLength - CompressedLength;
		}
		else
		{
			// No hits, no displacement
			OutSpringDisplacements[Index] = 0.f;
		}
	}
}

void FSimpleSuspensionHelpers::ComputeSuspensionDisplacements(const FSimpleSuspensionParams& SuspensionParams, const FSimpleSuspensionRaycast& RaycastFunction, FSimpleSuspensionState InOutSuspensionState)
{
	FSimpleSuspensionHelpers::ComputeSuspensionDisplacements(InOutSuspensionState.WorldSpringOrigins, SuspensionParams.SpringParams, InOutSuspensionState.WorldSuspensionNormal, RaycastFunction, InOutSuspensionState.SpringDisplacements);
}

float FSimpleSuspensionHelpers::ComputeSpringForce(const float SpringStiffness, const float SpringDamping, const float SpringDisplacement, const float SpringVelocity)
{
	const float StiffnessForce = SpringDisplacement * SpringStiffness;
	const float DampingForce = SpringDisplacement > UE_SMALL_NUMBER ? SpringVelocity * SpringDamping : 0.f;
	return StiffnessForce + DampingForce;
}

void FSimpleSuspensionHelpers::ComputeSuspensionForces(const FVector& LinearVelocity, const FVector& AngularVelocityRad, const FVector& WorldCenterOfMass, const FVector& WorldSuspensionNormal, const TArray<FVector> WorldSuspensionOrigins, const TArray<FSimpleSuspensionSpringParams>& SpringParams, const TArray<float>& SpringDisplacements, FVector& OutTotalForce, FVector& OutTotalTorque, TArray<float>& OutSpringForces)
{
	ensure(SpringParams.Num() == SpringDisplacements.Num());

	OutTotalForce = FVector::ZeroVector;
	OutTotalTorque = FVector::ZeroVector;
	const int32 Count = SpringParams.Num();
	for (int32 Index = 0; Index < Count; ++Index)
	{
		const FSimpleSuspensionSpringParams& Params = SpringParams[Index];
		const float SpringDisplacement = SpringDisplacements[Index];
		const float CompressedLength = Params.Length - SpringDisplacement;

		// Find the velocity of the endpoint of the spring
		const FVector WorldOrigin = WorldSuspensionOrigins[Index];
		const FVector COMOrigin = WorldOrigin - WorldCenterOfMass;
		const FVector SupportLinearVelocity = LinearVelocity + FVector::CrossProduct(AngularVelocityRad, COMOrigin);
		const float SpringVelocity = -FVector::DotProduct(WorldSuspensionNormal, SupportLinearVelocity);

		// Compute contribution of spring to total force and torque
		OutSpringForces[Index] = ComputeSpringForce(Params.Stiffness, Params.Damping, SpringDisplacement, SpringVelocity);

		//OutSpringForces[Index] = ComputeSpringForce(Params.Stiffness, Params.Damping, SpringDisplacement, SpringVelocity);
		const FVector SupportForce = WorldSuspensionNormal * OutSpringForces[Index];
		const FVector SupportTorque = FVector::CrossProduct(COMOrigin, SupportForce);
		OutTotalForce += SupportForce;
		OutTotalTorque += SupportTorque;
	}
}

void FSimpleSuspensionHelpers::ComputeSuspensionForces(const FVector& LinearVelocity, const FVector& AngularVelocityRad, const FSimpleSuspensionState& SuspensionState, const TArray<FSimpleSuspensionSpringParams>& SpringParams, FSimpleSuspensionState& OutSuspensionState)
{
	FSimpleSuspensionHelpers::ComputeSuspensionForces(LinearVelocity, AngularVelocityRad, SuspensionState.WorldCenterOfMass, SuspensionState.WorldSuspensionNormal, SuspensionState.WorldSpringOrigins, SpringParams, OutSuspensionState.SpringDisplacements, OutSuspensionState.TotalForce, OutSuspensionState.TotalTorqueRad, OutSuspensionState.SpringForces);
}

void FSimpleSuspensionHelpers::IntegrateSpring(const float DeltaTime, const float SpringDisplacement, const float SpringVelocity, const FSimpleSuspensionSpringParams& SpringParams, const float SprungMass, float& OutNewSpringDisplacement, float& OutNewSpringVelocity)
{
	//
	// TODO: Verify and test all this math! Not being used atm, so
	//       didn't bother to verify yet...
	//
	const float Denominator = SprungMass + (DeltaTime * (SpringParams.Damping + (SpringParams.Stiffness * DeltaTime)));
	if (Denominator > UE_SMALL_NUMBER)
	{
		// We do each of these integrations implicitly. We do them separately
		// before assigning return values to allow for inline integration
		// (ie, the caller doesn't own before/after state, just wants updated
		// values).
		const float DenominatorInv = 1.f / Denominator;
		const float NewDisplacement = DenominatorInv * ((SpringParams.Damping * SpringDisplacement * DeltaTime) + (SprungMass * (SpringDisplacement + (DeltaTime * SpringVelocity))));
		const float NewVelocity = DenominatorInv * ((SprungMass * SpringVelocity) - (DeltaTime * SpringParams.Stiffness * SpringDisplacement));
		OutNewSpringDisplacement = NewDisplacement;
		OutNewSpringVelocity = NewVelocity;
	}
	else
	{
		// Whatever caller is trying to do, it's not gonna work.
		OutNewSpringDisplacement = SpringDisplacement;
		OutNewSpringVelocity = SpringVelocity;
	}
}

void FSimpleSuspensionHelpers::IntegrateSprings(const float DeltaTime, const TArray<float>& SpringDisplacements, const TArray<float>& SpringVelocities, const TArray<FSimpleSuspensionSpringParams>& SuspensionParams, const TArray<float>& SprungMasses, TArray<float>& OutNewSpringDisplacements, TArray<float>& OutNewSpringVelocities)
{
	const int32 Count = SuspensionParams.Num();
	ensure(OutNewSpringDisplacements.Num() == Count);
	ensure(OutNewSpringVelocities.Num() == Count);
	ensure(SpringDisplacements.Num() == Count);
	ensure(SpringVelocities.Num() == Count);
	ensure(SprungMasses.Num() == Count);
	for (int32 Index = 0; Index < Count; ++Index)
	{
		IntegrateSpring(DeltaTime, SpringDisplacements[Index], SpringVelocities[Index], SuspensionParams[Index], SprungMasses[Index], OutNewSpringDisplacements[Index], OutNewSpringVelocities[Index]);
	}
}

bool FSimpleSuspensionHelpers::ErrCheck(const bool bCondition, const FString& Message, FString* ErrMsg)
{
	if (ErrMsg != nullptr && !bCondition) { *ErrMsg = Message; }
	return bCondition;
}
