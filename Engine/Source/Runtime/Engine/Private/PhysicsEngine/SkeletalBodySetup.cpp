// Copyright Epic Games, Inc. All Rights Reserved.

#include "PhysicsEngine/SkeletalBodySetup.h"
#include "PhysicsEngine/PhysicsAsset.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SkeletalBodySetup)

#if WITH_EDITOR
void USkeletalBodySetup::PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent)
{
	UPhysicsAsset* OwningPhysAsset = Cast<UPhysicsAsset>(GetOuter());

	if (PropertyChangedEvent.Property == nullptr || !OwningPhysAsset)
	{
		return;
	}

	if (FPhysicalAnimationProfile* PhysProfile = FindPhysicalAnimationProfile(OwningPhysAsset->CurrentPhysicalAnimationProfileName))
	{
		//changed any setting so copy dummy UI into profile location
		PhysProfile->PhysicalAnimationData = CurrentPhysicalAnimationProfile.PhysicalAnimationData;
	}

	OwningPhysAsset->RefreshPhysicsAssetChange();
}

FName USkeletalBodySetup::GetCurrentPhysicalAnimationProfileName() const
{
	FName CurrentProfileName;
	if (UPhysicsAsset* OwningPhysAsset = Cast<UPhysicsAsset>(GetOuter()))
	{
		CurrentProfileName = OwningPhysAsset->CurrentPhysicalAnimationProfileName;
	}

	return CurrentProfileName;
}

void USkeletalBodySetup::AddPhysicalAnimationProfile(FName ProfileName)
{
	FPhysicalAnimationProfile* NewProfile = new (PhysicalAnimationData) FPhysicalAnimationProfile();
	NewProfile->ProfileName = ProfileName;
}

void USkeletalBodySetup::RemovePhysicalAnimationProfile(FName ProfileName)
{
	for (int32 ProfileIdx = 0; ProfileIdx < PhysicalAnimationData.Num(); ++ProfileIdx)
	{
		if (PhysicalAnimationData[ProfileIdx].ProfileName == ProfileName)
		{
			PhysicalAnimationData.RemoveAtSwap(ProfileIdx--);
		}
	}
}

void USkeletalBodySetup::UpdatePhysicalAnimationProfiles(const TArray<FName>& Profiles)
{
	for (int32 ProfileIdx = 0; ProfileIdx < PhysicalAnimationData.Num(); ++ProfileIdx)
	{
		if (Profiles.Contains(PhysicalAnimationData[ProfileIdx].ProfileName) == false)
		{
			PhysicalAnimationData.RemoveAtSwap(ProfileIdx--);
		}
	}
}

void USkeletalBodySetup::DuplicatePhysicalAnimationProfile(FName DuplicateFromName, FName DuplicateToName)
{
	for (FPhysicalAnimationProfile& ProfileHandle : PhysicalAnimationData)
	{
		if (ProfileHandle.ProfileName == DuplicateFromName)
		{
			FPhysicalAnimationProfile* Duplicate = new (PhysicalAnimationData) FPhysicalAnimationProfile(ProfileHandle);
			Duplicate->ProfileName = DuplicateToName;
			break;
		}
	}
}

void USkeletalBodySetup::RenamePhysicalAnimationProfile(FName CurrentName, FName NewName)
{
	for (FPhysicalAnimationProfile& ProfileHandle : PhysicalAnimationData)
	{
		if (ProfileHandle.ProfileName == CurrentName)
		{
			ProfileHandle.ProfileName = NewName;
		}
	}
}

#endif
