// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/AutomationTest.h"
#include "Physics/Tests/PhysicsTestHelpers.h"
#include "Engine/StaticMeshActor.h"
#include "Components/StaticMeshComponent.h"
#include "Engine/StaticMesh.h"

#if WITH_EDITOR
#endif

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FPhysicsBodyInstanceTest, "System.Physics.Interface.BodyInstance", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)

bool FPhysicsBodyInstanceTest::RunTest(const FString& Parameters)
{
	UWorld* World = FPhysicsTestHelpers::GetWorld();
	check(World);

	AStaticMeshActor* SMA = Cast<AStaticMeshActor>(World->SpawnActor(AStaticMeshActor::StaticClass()));
	check(SMA);

	UStaticMeshComponent* SMC = SMA->GetStaticMeshComponent();
	check(SMC);

	UStaticMesh* Mesh = Cast<UStaticMesh>(StaticLoadObject(UStaticMesh::StaticClass(), NULL, TEXT("/Engine/EditorMeshes/EditorCube")));
	SMC->SetMobility(EComponentMobility::Movable);
	SMC->SetSimulatePhysics(true);
	SMC->SetStaticMesh(Mesh);

	// TODO: Physics actor handle will be changing to a ptr!
	FBodyInstance& BI = SMC->BodyInstance;
	FPhysicsActorHandle PH = BI.GetPhysicsActor();

	check(FPhysicsInterface::IsValid(PH));
	check(FPhysicsInterface::IsRigidBody(PH));

	return true;
}

