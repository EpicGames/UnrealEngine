// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/AutomationTest.h"
#include "Physics/Experimental/PhysInterface_Chaos.h"
#include "PhysicsInterfaceTypesCore.h"
#if WITH_EDITOR
#endif

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FPhysicsInterfaceTest, "System.Physics.Interface.ObjectCreation", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)

bool FPhysicsInterfaceTest::RunTest(const FString& Parameters)
{
	FPhysicsActorHandle ActorHandle;
	FPhysicsInterface::CreateActor(FActorCreationParams(), ActorHandle);
	// TODO: Make some assertions
	FPhysicsInterface::ReleaseActor(ActorHandle);
	// TODO: Make some more assertions
	return true;
}

