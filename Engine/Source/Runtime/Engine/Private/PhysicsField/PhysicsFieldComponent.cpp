// Copyright Epic Games, Inc. All Rights Reserved.

#include "PhysicsField/PhysicsFieldComponent.h"
#include "ChaosStats.h"
#include "Engine/Engine.h"
#include "Engine/World.h"
#include "Field/FieldSystemNodes.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "GlobalShader.h"
#include "HAL/LowLevelMemTracker.h"
#include "PipelineStateCache.h"
#include "ShaderParameterUtils.h"
#include "ProfilingDebugging/RealtimeGPUProfiler.h"
#include "RHIBreadcrumbs.h"
#include "RHIContext.h"
#include "RenderingThread.h"
#include "SceneInterface.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PhysicsFieldComponent)

/**
*	Stats
*
*/

DECLARE_STATS_GROUP(TEXT("PhysicsFields"), STATGROUP_PhysicsFields, STATCAT_Advanced);

DECLARE_CYCLE_STAT(TEXT("Create Scene Proxy [GT]"), STAT_PhysicsFields_CreateSceneProxy, STATGROUP_PhysicsFields);
DECLARE_CYCLE_STAT(TEXT("Send Render Data [GT]"), STAT_PhysicsFields_SendRenderData, STATGROUP_PhysicsFields);
DECLARE_CYCLE_STAT(TEXT("UpdateResource [RT]"), STAT_PhysicsFields_UpdateResource_RT, STATGROUP_PhysicsFields);
DECLARE_GPU_STAT(PhysicsFields_ClearClipmap);
DECLARE_GPU_STAT(PhysicsFields_BuildClipmap);
DECLARE_GPU_STAT(PhysicsFields_UpdateBuffers);

/**
*	Console variables
* 
*/
DEFINE_LOG_CATEGORY_STATIC(LogGlobalField, Log, All);

/** Boolean to check if we need to build or not the clipmap */
int32 GPhysicsFieldBuildClipmap = 1;
FAutoConsoleVariableRef CVarPhysicsFieldBuildClipmap(
	TEXT("r.PhysicsField.BuildClipmap"),
	GPhysicsFieldBuildClipmap,
	TEXT("Build the Physics field clipmap"),
	ECVF_Scalability | ECVF_RenderThreadSafe);

/** Clipmap enable/disable */
static TAutoConsoleVariable<int32> CVarPhysicsFieldEnableClipmap(
	TEXT("r.PhysicsField.EnableField"),
	1,
	TEXT("Enable/Disable the Physics field clipmap"),
	ECVF_Scalability | ECVF_RenderThreadSafe);

/** Clipmap max disatnce */
float GPhysicsFieldClipmapDistance = 10000;
FAutoConsoleVariableRef CVarPhysicsFieldClipmapDistance(
	TEXT("r.PhysicsField.ClipmapDistance"),
	GPhysicsFieldClipmapDistance,
	TEXT("Max distance from the clipmap center"),
	ECVF_RenderThreadSafe
);

/** Number of used clipmaps */
int32 GPhysicsFieldClipmapCount = 4;
FAutoConsoleVariableRef CVarPhysicsFieldClipmapCount(
	TEXT("r.PhysicsField.ClipmapCount"),
	GPhysicsFieldClipmapCount,
	TEXT("Number of clipmaps used for the physics field"),
	ECVF_RenderThreadSafe
);

/** Exponent used to compute each clipmaps distance */
float GPhysicsFieldClipmapExponent = 2;
FAutoConsoleVariableRef CVarPhysicsFieldClipmapExponent(
	TEXT("r.PhysicsField.ClipmapExponent"),
	GPhysicsFieldClipmapExponent,
	TEXT("Exponent used to derive each clipmap's size, together with r.PhysicsField.ClipmapDistance"),
	ECVF_RenderThreadSafe
);

/** Resolution of each clipmaps */
int32 GPhysicsFieldClipmapResolution = 32;
FAutoConsoleVariableRef CVarPhysicsFieldClipmapResolution(
	TEXT("r.PhysicsField.ClipmapResolution"),
	GPhysicsFieldClipmapResolution,
	TEXT("Resolution of the physics field.  Higher values increase fidelity but also increase memory and composition cost."),
	ECVF_RenderThreadSafe
);

/** Single Target Limit */
int32 GPhysicsFieldSingleTarget = 0;
FAutoConsoleVariableRef CVarPhysicsFieldSingleTarget(
	TEXT("r.PhysicsField.SingleTarget"),
	GPhysicsFieldSingleTarget,
	TEXT("Limnit the physics field build to only one target, the linear force"),
	ECVF_RenderThreadSafe
);

/** Spatial culling */
int32 GPhysicsFieldEnableCulling = 1;
FAutoConsoleVariableRef CVarPhysicsFieldEnableCulling(
	TEXT("r.PhysicsField.EnableCulling"),
	GPhysicsFieldEnableCulling,
	TEXT("Enable the spatial culling based on the field nodes bounds"),
	ECVF_RenderThreadSafe
);


/**
*	Resource Utilities
*/

template<typename BufferType, int ElementSize, EPixelFormat PixelFormat>
void InitInternalBuffer(FRHICommandListBase& RHICmdList, const TCHAR* DebugName, const uint32 ElementCount, FRWBuffer& OutputBuffer)
{
	if (ElementCount > 0)
	{
		const uint32 BufferCount = ElementCount * ElementSize;
		const uint32 BufferBytes = sizeof(BufferType) * BufferCount;
		
		{
			LLM_SCOPE(ELLMTag::Physics);
			OutputBuffer.Initialize(RHICmdList, DebugName, sizeof(BufferType), BufferCount, PixelFormat, BUF_Static);
		}

		if (OutputBuffer.UAV)
		{
			FRHIComputeCommandList& RHICmdListCompute = FRHIComputeCommandList::Get(RHICmdList);

			RHICmdListCompute.Transition(FRHITransitionInfo(OutputBuffer.UAV, ERHIAccess::Unknown, ERHIAccess::UAVCompute));
			if (PixelFormat == EPixelFormat::PF_R32_FLOAT || PixelFormat == EPixelFormat::PF_A32B32G32R32F)
			{
				RHICmdListCompute.ClearUAVFloat(OutputBuffer.UAV, FVector4f(ForceInitToZero));
			}
			else
			{
				RHICmdListCompute.ClearUAVUint(OutputBuffer.UAV, FUintVector4(ForceInitToZero));
			}
			RHICmdListCompute.Transition(FRHITransitionInfo(OutputBuffer.UAV, ERHIAccess::UAVCompute, ERHIAccess::SRVCompute));
		}
	}
}

template<typename BufferType, int ElementSize, EPixelFormat PixelFormat>
void UpdateInternalBuffer(FRHICommandListBase& RHICmdList, const TCHAR* DebugName, const uint32 ElementCount, const BufferType* InputData, FRWBuffer& OutputBuffer, const bool bInitField = false, ERHIAccess InitAccess = ERHIAccess::SRVMask)
{
	if (ElementCount > 0 && InputData)
	{
		const uint32 BufferCount = ElementCount * ElementSize;
		const uint32 BufferBytes = sizeof(BufferType) * BufferCount;

		if (bInitField)
		{
			LLM_SCOPE(ELLMTag::Physics);
			OutputBuffer.Initialize(RHICmdList, DebugName, sizeof(BufferType), BufferCount, PixelFormat, InitAccess, BUF_Static);
		}

		void* OutputData = RHICmdList.LockBuffer(OutputBuffer.Buffer, 0, BufferBytes, RLM_WriteOnly);

		FMemory::Memcpy(OutputData, InputData, BufferBytes);
		RHICmdList.UnlockBuffer(OutputBuffer.Buffer);
	}
}

FVector MinVector(const FVector& VectorA, const FVector& VectorB)
{
	return FVector(FMath::Min(VectorA.X, VectorB.X), FMath::Min(VectorA.Y, VectorB.Y), FMath::Min(VectorA.Z, VectorB.Z));
}

FVector MaxVector(const FVector& VectorA, const FVector& VectorB)
{
	return FVector(FMath::Max(VectorA.X, VectorB.X), FMath::Max(VectorA.Y, VectorB.Y), FMath::Max(VectorA.Z, VectorB.Z));
}

/**
*	Clipmap construction
*/

class FBuildPhysicsFieldClipmapCS : public FGlobalShader
{
	DECLARE_SHADER_TYPE(FBuildPhysicsFieldClipmapCS, Global)

public:

	static const uint32 ThreadGroupSize = 64;

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return !IsMobilePlatform(Parameters.Platform) && IsFeatureLevelSupported(Parameters.Platform, ERHIFeatureLevel::SM5);
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("BUILD_FIELD_THREAD_GROUP_SIZE"), ThreadGroupSize);
		OutEnvironment.SetDefine(TEXT("MAX_PHYSICS_FIELD_TARGETS"), MAX_PHYSICS_FIELD_TARGETS);
	}

	FBuildPhysicsFieldClipmapCS(const ShaderMetaType::CompiledShaderInitializerType& Initializer)
		: FGlobalShader(Initializer)
	{
		NodesParams.Bind(Initializer.ParameterMap, TEXT("NodesParams"));
		NodesOffsets.Bind(Initializer.ParameterMap, TEXT("NodesOffsets"));
		TargetsOffsets.Bind(Initializer.ParameterMap, TEXT("TargetsOffsets"));
		FieldClipmap.Bind(Initializer.ParameterMap, TEXT("FieldClipmap"));

		CellsOffsets.Bind(Initializer.ParameterMap, TEXT("CellsOffsets"));
		CellsMin.Bind(Initializer.ParameterMap, TEXT("CellsMin"));
		CellsMax.Bind(Initializer.ParameterMap, TEXT("CellsMax"));

		ClipmapResolution.Bind(Initializer.ParameterMap, TEXT("ClipmapResolution"));
		ClipmapDistance.Bind(Initializer.ParameterMap, TEXT("ClipmapDistance"));
		ClipmapCenter.Bind(Initializer.ParameterMap, TEXT("ClipmapCenter"));
		ClipmapCount.Bind(Initializer.ParameterMap, TEXT("ClipmapCount"));
		ClipmapExponent.Bind(Initializer.ParameterMap, TEXT("ClipmapExponent"));

		TargetsIndex.Bind(Initializer.ParameterMap, TEXT("TargetsIndex"));
		ValidTargets.Bind(Initializer.ParameterMap, TEXT("ValidTargets"));
		ValidCount.Bind(Initializer.ParameterMap, TEXT("ValidCount"));
		TimeSeconds.Bind(Initializer.ParameterMap, TEXT("TimeSeconds"));
		NumCells.Bind(Initializer.ParameterMap, TEXT("NumCells"));
		TargetCount.Bind(Initializer.ParameterMap, TEXT("TargetCount"));
	}

	FBuildPhysicsFieldClipmapCS()
	{
	}

	void SetParameters(FRHIBatchedShaderParameters& BatchedParameters, FPhysicsFieldResource* FieldResource, const float InTimeSeconds, const int32 InNumCells)
	{
		if (FieldResource)
		{
			SetSRVParameter(BatchedParameters, NodesParams, FieldResource->NodesParams.SRV);
			SetSRVParameter(BatchedParameters, NodesOffsets, FieldResource->NodesOffsets.SRV);
			SetSRVParameter(BatchedParameters, TargetsOffsets, FieldResource->TargetsOffsets.SRV);
			SetUAVParameter(BatchedParameters, FieldClipmap, FieldResource->ClipmapBuffer.UAV);

			SetSRVParameter(BatchedParameters, CellsOffsets, FieldResource->CellsOffsets.SRV);
			SetSRVParameter(BatchedParameters, CellsMin, FieldResource->CellsMin.SRV);
			SetSRVParameter(BatchedParameters, CellsMax, FieldResource->CellsMax.SRV);

			SetShaderValue(BatchedParameters, ClipmapResolution, FieldResource->FieldInfos.ClipmapResolution);
			SetShaderValue(BatchedParameters, ClipmapDistance, FieldResource->FieldInfos.ClipmapDistance);
			SetShaderValue(BatchedParameters, ClipmapCount, FieldResource->FieldInfos.ClipmapCount);
			SetShaderValue(BatchedParameters, ClipmapCenter, (FVector3f)FieldResource->FieldInfos.ClipmapCenter);
			SetShaderValue(BatchedParameters, ClipmapExponent, FieldResource->FieldInfos.ClipmapExponent);

			SetShaderValue(BatchedParameters, ValidTargets, FieldResource->FieldInfos.ValidTargets);
			SetShaderValue(BatchedParameters, ValidCount, FieldResource->FieldInfos.ValidCount);
			SetShaderValue(BatchedParameters, TargetsIndex, FieldResource->FieldInfos.PhysicsTargets);
			SetShaderValue(BatchedParameters, TimeSeconds, InTimeSeconds);
			SetShaderValue(BatchedParameters, NumCells, InNumCells);
			SetShaderValue(BatchedParameters, TargetCount, FieldResource->FieldInfos.TargetCount);
		}
	}

	void UnsetParameters(FRHIBatchedShaderUnbinds& BatchedUnbinds)
	{
		UnsetUAVParameter(BatchedUnbinds, FieldClipmap);
	}

private:
	
	LAYOUT_FIELD(FShaderResourceParameter, NodesParams);
	LAYOUT_FIELD(FShaderResourceParameter, NodesOffsets);
	LAYOUT_FIELD(FShaderResourceParameter, TargetsOffsets);
	LAYOUT_FIELD(FShaderResourceParameter, FieldClipmap);

	LAYOUT_FIELD(FShaderResourceParameter, CellsOffsets);
	LAYOUT_FIELD(FShaderResourceParameter, CellsMin);
	LAYOUT_FIELD(FShaderResourceParameter, CellsMax);

	LAYOUT_FIELD(FShaderParameter, ClipmapResolution);
	LAYOUT_FIELD(FShaderParameter, ClipmapDistance);
	LAYOUT_FIELD(FShaderParameter, ClipmapCenter);
	LAYOUT_FIELD(FShaderParameter, ClipmapCount);
	LAYOUT_FIELD(FShaderParameter, ClipmapExponent);

	LAYOUT_FIELD(FShaderParameter, TargetsIndex);
	LAYOUT_FIELD(FShaderParameter, ValidTargets);
	LAYOUT_FIELD(FShaderParameter, ValidCount);
	LAYOUT_FIELD(FShaderParameter, TimeSeconds);
	LAYOUT_FIELD(FShaderParameter, NumCells);
	LAYOUT_FIELD(FShaderParameter, TargetCount);
};

IMPLEMENT_SHADER_TYPE(, FBuildPhysicsFieldClipmapCS, TEXT("/Engine/Private/PhysicsFieldBuilder.usf"), TEXT("BuildPhysicsFieldClipmapCS"), SF_Compute);

/**
*	Clipmap reset
*/

class FResetPhysicsFieldClipmapCS : public FGlobalShader
{
	DECLARE_SHADER_TYPE(FResetPhysicsFieldClipmapCS, Global)

public:

	static const uint32 ThreadGroupSize = 64;

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return !IsMobilePlatform(Parameters.Platform) && IsFeatureLevelSupported(Parameters.Platform, ERHIFeatureLevel::SM5);
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("BUILD_FIELD_THREAD_GROUP_SIZE"), ThreadGroupSize);
		OutEnvironment.SetDefine(TEXT("MAX_PHYSICS_FIELD_TARGETS"), MAX_PHYSICS_FIELD_TARGETS);
	}

	FResetPhysicsFieldClipmapCS(const ShaderMetaType::CompiledShaderInitializerType& Initializer)
		: FGlobalShader(Initializer)
	{
		FieldClipmap.Bind(Initializer.ParameterMap, TEXT("FieldClipmap"));
		NumCells.Bind(Initializer.ParameterMap, TEXT("NumCells"));

		CellsOffsets.Bind(Initializer.ParameterMap, TEXT("CellsOffsets"));
		CellsMin.Bind(Initializer.ParameterMap, TEXT("CellsMin"));
		CellsMax.Bind(Initializer.ParameterMap, TEXT("CellsMax"));

		ClipmapResolution.Bind(Initializer.ParameterMap, TEXT("ClipmapResolution"));
		ClipmapCount.Bind(Initializer.ParameterMap, TEXT("ClipmapCount"));

		TargetsIndex.Bind(Initializer.ParameterMap, TEXT("TargetsIndex"));
		ValidTargets.Bind(Initializer.ParameterMap, TEXT("ValidTargets"));
		ValidCount.Bind(Initializer.ParameterMap, TEXT("ValidCount"));
		TargetCount.Bind(Initializer.ParameterMap, TEXT("TargetCount"));
	}

	FResetPhysicsFieldClipmapCS()
	{
	}

	void SetParameters(FRHIBatchedShaderParameters& BatchedParameters, FPhysicsFieldResource* FieldResource, const float InTimeSeconds, const int32 InNumCells)
	{
		if (FieldResource)
		{
			SetUAVParameter(BatchedParameters, FieldClipmap, FieldResource->ClipmapBuffer.UAV);
			SetShaderValue(BatchedParameters, NumCells, InNumCells);

			SetSRVParameter(BatchedParameters, CellsOffsets, FieldResource->CellsOffsets.SRV);
			SetSRVParameter(BatchedParameters, CellsMin, FieldResource->CellsMin.SRV);
			SetSRVParameter(BatchedParameters, CellsMax, FieldResource->CellsMax.SRV);

			SetShaderValue(BatchedParameters, ClipmapResolution, FieldResource->FieldInfos.ClipmapResolution);
			SetShaderValue(BatchedParameters, ClipmapCount, FieldResource->FieldInfos.ClipmapCount);

			SetShaderValue(BatchedParameters, ValidTargets, FieldResource->FieldInfos.ValidTargets);
			SetShaderValue(BatchedParameters, ValidCount, FieldResource->FieldInfos.ValidCount);
			SetShaderValue(BatchedParameters, TargetsIndex, FieldResource->FieldInfos.PhysicsTargets);
			SetShaderValue(BatchedParameters, TargetCount, FieldResource->FieldInfos.TargetCount);
		}
	}

	void UnsetParameters(FRHIBatchedShaderUnbinds& BatchedUnbinds)
	{
		UnsetUAVParameter(BatchedUnbinds, FieldClipmap);
	}

private:

	LAYOUT_FIELD(FShaderResourceParameter, FieldClipmap);
	LAYOUT_FIELD(FShaderParameter, NumCells);

	LAYOUT_FIELD(FShaderResourceParameter, CellsOffsets);
	LAYOUT_FIELD(FShaderResourceParameter, CellsMin);
	LAYOUT_FIELD(FShaderResourceParameter, CellsMax);

	LAYOUT_FIELD(FShaderParameter, ClipmapResolution);
	LAYOUT_FIELD(FShaderParameter, ClipmapCount);

	LAYOUT_FIELD(FShaderParameter, TargetsIndex);
	LAYOUT_FIELD(FShaderParameter, ValidTargets);
	LAYOUT_FIELD(FShaderParameter, ValidCount);
	LAYOUT_FIELD(FShaderParameter, TargetCount);
};

IMPLEMENT_SHADER_TYPE(, FResetPhysicsFieldClipmapCS, TEXT("/Engine/Private/PhysicsFieldReset.usf"), TEXT("ResetPhysicsFieldClipmapCS"), SF_Compute);


/**
*	FPhysicsFieldResource
*/

FPhysicsFieldResource::FPhysicsFieldResource(const int32 TargetCount, const TArray<EFieldPhysicsType>& TargetTypes, 
		const FPhysicsFieldInfos::BufferOffsets& VectorTargets, const FPhysicsFieldInfos::BufferOffsets& ScalarTargets,
	    const FPhysicsFieldInfos::BufferOffsets& IntegerTargets, const FPhysicsFieldInfos::BufferOffsets& PhysicsTargets,
		const TStaticArray<UE::Core::TAlignedElement<FIntVector4, 16>, MAX_PHYSICS_FIELD_TARGETS>& PhysicsBounds, const bool bBuildClipmap) : FRenderResource()
{
	FieldInfos.TargetCount = TargetCount;
	FieldInfos.TargetTypes = TargetTypes;
	FieldInfos.VectorTargets = VectorTargets;
	FieldInfos.ScalarTargets = ScalarTargets;
	FieldInfos.IntegerTargets = IntegerTargets;
	FieldInfos.PhysicsTargets = PhysicsTargets;
	FieldInfos.PhysicsBounds = PhysicsBounds;
	FieldInfos.bBuildClipmap = bBuildClipmap;
	FieldInfos.bShowFields = false;

	FieldInfos.ClipmapExponent = GPhysicsFieldClipmapExponent;
	FieldInfos.ClipmapCount = GPhysicsFieldClipmapCount;
	FieldInfos.ClipmapDistance = GPhysicsFieldClipmapDistance;
	FieldInfos.ClipmapResolution = GPhysicsFieldClipmapResolution;

	const uint32 CellsCount = FieldInfos.ClipmapCount * EFieldPhysicsType::Field_PhysicsType_Max;
	FieldInfos.CellsOffsets.Init(0, CellsCount + 1);
	FieldInfos.CellsMin.Init(FIntVector4(0), CellsCount);
	FieldInfos.CellsMax.Init(FIntVector4(FieldInfos.ClipmapResolution), CellsCount);

	for (uint32 BoundIndex = 0; BoundIndex < MAX_PHYSICS_FIELD_TARGETS; ++BoundIndex)
	{
		FieldInfos.BoundsOffsets[BoundIndex] = 0;
	}
	FieldInfos.TimeSeconds = 0.0;
}

void FPhysicsFieldResource::InitRHI(FRHICommandListBase& RHICmdList)
{
	SCOPE_CYCLE_COUNTER(STAT_PhysicsFields_UpdateResource_RT);

	const int32 DatasCount = FieldInfos.ClipmapCount * FieldInfos.TargetCount;
	InitInternalBuffer<int32, 1, EPixelFormat::PF_R32_SINT>(RHICmdList, TEXT("PhysicsField_TargetOffsets"), EFieldPhysicsType::Field_PhysicsType_Max + 1, TargetsOffsets);

	InitInternalBuffer<FVector4f, 1, EPixelFormat::PF_A32B32G32R32F>(RHICmdList, TEXT("PhysicsField_BoundsMin"), 1, BoundsMin);
	InitInternalBuffer<FVector4f, 1, EPixelFormat::PF_A32B32G32R32F>(RHICmdList, TEXT("PhysicsField_BoundsMax"), 1, BoundsMax);

	InitInternalBuffer<float, 1, EPixelFormat::PF_R32_FLOAT>(RHICmdList, TEXT("PhysicsField_NodesParams"), 1, NodesParams);
	InitInternalBuffer<int32, 1, EPixelFormat::PF_R32_SINT>(RHICmdList, TEXT("PhysicsField_NodesOffsets"), 1, NodesOffsets);

	if (FieldInfos.bBuildClipmap)
	{
		const int32 CellsCount = FieldInfos.ClipmapCount * EFieldPhysicsType::Field_PhysicsType_Max;
		InitInternalBuffer<float, 1, EPixelFormat::PF_R32_FLOAT>(RHICmdList, TEXT("PhysicsField_ClipmapBuffer"), FieldInfos.ClipmapResolution * FieldInfos.ClipmapResolution * FieldInfos.ClipmapResolution * DatasCount, ClipmapBuffer);
		InitInternalBuffer<int32, 1, EPixelFormat::PF_R32_SINT>(RHICmdList, TEXT("PhysicsField_CellsOffsets"), CellsCount + 1, CellsOffsets);
		InitInternalBuffer<FIntVector4, 1, EPixelFormat::PF_R32G32B32A32_UINT>(RHICmdList, TEXT("PhysicsField_CellsMin"), CellsCount, CellsMin);
		InitInternalBuffer<FIntVector4, 1, EPixelFormat::PF_R32G32B32A32_UINT>(RHICmdList, TEXT("PhysicsField_CellsMax"), CellsCount, CellsMax );

		FRHIComputeCommandList& RHICmdListCompute = FRHIComputeCommandList::Get(RHICmdList);
		RHICmdListCompute.Transition(FRHITransitionInfo(ClipmapBuffer.UAV, ERHIAccess::SRVCompute, ERHIAccess::SRVMask));
	}
}

void FPhysicsFieldResource::ReleaseRHI()
{
	NodesParams.Release();
	NodesOffsets.Release();
	TargetsOffsets.Release();
	BoundsMin.Release();
	BoundsMax.Release();

	if (FieldInfos.bBuildClipmap)
	{
		ClipmapBuffer.Release();
		CellsOffsets.Release();
		CellsMin.Release();
		CellsMax.Release();
	}
}

void FPhysicsFieldResource::UpdateBounds(const TArray<FVector>& TargetsMin, const TArray<FVector>& TargetsMax, const TStaticArray<int32, EFieldPhysicsType::Field_PhysicsType_Max + 1>& TargetOffsets,
	const TStaticArray<int32, EFieldPhysicsType::Field_PhysicsType_Max + 1>& BoundsOffsets)
{
	const uint32 CellsCount = FieldInfos.ClipmapCount * EFieldPhysicsType::Field_PhysicsType_Max;
	FieldInfos.CellsOffsets.Init(0, CellsCount + 1);
	FieldInfos.CellsMin.Init(FIntVector4(0), CellsCount);
	FieldInfos.CellsMax.Init(FIntVector4(FieldInfos.ClipmapResolution), CellsCount);

	if(GPhysicsFieldEnableCulling == 1)
	{ 
		FieldInfos.ValidCount = 0;
		for (auto& TargetType : FieldInfos.TargetTypes)
		{
			if((TargetOffsets[TargetType+1]- TargetOffsets[TargetType]) > 0)
			{ 
				for (int32 ClipmapIndex = 0; ClipmapIndex < FieldInfos.ClipmapCount; ++ClipmapIndex)
				{
					const float ClipmapExtent = FieldInfos.ClipmapDistance * FMath::Pow(
						(float)FieldInfos.ClipmapExponent, (float)(ClipmapIndex + 1 - FieldInfos.ClipmapCount));

					const float CellSize = 2.0 * ClipmapExtent / (FieldInfos.ClipmapResolution - 1.0);

					const FVector GlobalMin = FieldInfos.ClipmapCenter - FVector(ClipmapExtent);
					const FVector GlobalMax = FieldInfos.ClipmapCenter + FVector(ClipmapExtent);

					const int32 CellIndex = FieldInfos.ValidCount * FieldInfos.ClipmapCount + ClipmapIndex;

					const FVector& MinBound = TargetsMin[TargetType];
					const FVector& MaxBound = TargetsMax[TargetType];

					FieldInfos.CellsMin[CellIndex] = FIntVector4(0, 0, 0, 0);
					FieldInfos.CellsMax[CellIndex] = FIntVector4(0, 0, 0, 0);

					if (MaxBound.X > MinBound.X && MaxBound.Y > MinBound.Y && MaxBound.Z > MinBound.Z)
					{
						FVector LocalMin = MaxVector(GlobalMin, MinBound) - GlobalMin;
						FVector LocalMax = MinVector(GlobalMax, MaxBound) - GlobalMin;

						if (LocalMin.X < LocalMax.X && LocalMin.Y < LocalMax.Y && LocalMin.Z < LocalMax.Z)
						{
							FieldInfos.CellsMin[CellIndex] = FIntVector4(FMath::CeilToInt(LocalMin.X / CellSize - UE_KINDA_SMALL_NUMBER),
								FMath::CeilToInt(LocalMin.Y / CellSize - UE_KINDA_SMALL_NUMBER),
								FMath::CeilToInt(LocalMin.Z / CellSize - UE_KINDA_SMALL_NUMBER), 0);
							FieldInfos.CellsMax[CellIndex] = FIntVector4(FMath::CeilToInt(LocalMax.X / CellSize + UE_KINDA_SMALL_NUMBER),
								FMath::CeilToInt(LocalMax.Y / CellSize + UE_KINDA_SMALL_NUMBER),
								FMath::CeilToInt(LocalMax.Z / CellSize + UE_KINDA_SMALL_NUMBER), 0);

							FieldInfos.CellsOffsets[CellIndex + 1] = (FieldInfos.CellsMax[CellIndex].X - FieldInfos.CellsMin[CellIndex].X) * 
																	 (FieldInfos.CellsMax[CellIndex].Y - FieldInfos.CellsMin[CellIndex].Y) * 
																	 (FieldInfos.CellsMax[CellIndex].Z - FieldInfos.CellsMin[CellIndex].Z);

							const float CellsRatio = FieldInfos.CellsOffsets[CellIndex + 1] / FMath::Pow(FieldInfos.ClipmapResolution, 3.0);

								/*UE_LOGF(LogGlobalField, Log, "Field Bounds[%d][%d] = %d %d %d -> %d %d %d | Cells Ratio = %f | %d", TargetType, ClipmapIndex,
									FieldInfos.CellsMin[CellIndex].X, FieldInfos.CellsMin[CellIndex].Y, FieldInfos.CellsMin[CellIndex].Z,
									FieldInfos.CellsMax[CellIndex].X, FieldInfos.CellsMax[CellIndex].Y, FieldInfos.CellsMax[CellIndex].Z, CellsRatio, FieldInfos.CellsOffsets[CellIndex + 1]);*/
						}
					}
				}
				FieldInfos.ValidTargets[FieldInfos.ValidCount++] = TargetType;
			}
		}
	}
	for (uint32 CellsIndex = 1; CellsIndex < CellsCount + 1; ++CellsIndex)
	{
		FieldInfos.CellsOffsets[CellsIndex] += FieldInfos.CellsOffsets[CellsIndex - 1];
	}
}

void FPhysicsFieldResource::UpdateResource(FRHICommandList& RHICmdList, 
	const TStaticArray<int32, EFieldPhysicsType::Field_PhysicsType_Max + 1>& TargetsOffsetsDatas, const TArray<int32>& NodesOffsetsDatas, const TArray<float>& NodesParamsDatas,
	const TArray<FVector>& TargetsMinDatas, const TArray<FVector>& TargetsMaxDatas, const float TimeSeconds,
	const TArray<FVector4>& BoundsMinDatas, const TArray<FVector4>& BoundsMaxDatas, const TStaticArray<int32, EFieldPhysicsType::Field_PhysicsType_Max + 1>& BoundsOffsetsDatas)
{
	SCOPE_CYCLE_COUNTER(STAT_PhysicsFields_UpdateResource_RT);

	{
		RHI_BREADCRUMB_EVENT_STAT(RHICmdList, PhysicsFields_UpdateBuffers, "PhysicsFields_UpdateBuffers");

		UpdateInternalBuffer<float, 1, EPixelFormat::PF_R32_FLOAT>(RHICmdList, TEXT("PhysicsField_NodeParams"), NodesParamsDatas.Num(), NodesParamsDatas.GetData(), NodesParams, true);
		UpdateInternalBuffer<int32, 1, EPixelFormat::PF_R32_SINT>(RHICmdList, TEXT("PhysicsField_NodeOffset"), NodesOffsetsDatas.Num(), NodesOffsetsDatas.GetData(), NodesOffsets, true);
		UpdateInternalBuffer<int32, 1, EPixelFormat::PF_R32_SINT>(RHICmdList, TEXT("PhysicsField_TargetOffset"), TargetsOffsetsDatas.Num(), TargetsOffsetsDatas.GetData(), TargetsOffsets);

		FieldInfos.TimeSeconds = TimeSeconds;
		for (uint32 BoundIndex = 0; BoundIndex < EFieldPhysicsType::Field_PhysicsType_Max + 1; ++BoundIndex)
		{
			FieldInfos.BoundsOffsets[BoundIndex] = BoundsOffsetsDatas[BoundIndex];
		}
		// LWC_TODO: Perf pessimization
		// LWC_TODO: Precision loss
		UpdateInternalBuffer<FVector4f, 1, EPixelFormat::PF_A32B32G32R32F>(RHICmdList, TEXT("PhysicsField_BoundsMin"), BoundsMinDatas.Num(), UE::LWC::ConvertArrayType<FVector4f>(BoundsMinDatas).GetData(), BoundsMin, true);
		UpdateInternalBuffer<FVector4f, 1, EPixelFormat::PF_A32B32G32R32F>(RHICmdList, TEXT("PhysicsField_BoundsMax"), BoundsMaxDatas.Num(),  UE::LWC::ConvertArrayType<FVector4f>(BoundsMaxDatas).GetData(), BoundsMax, true);
	}

	if (FieldInfos.bBuildClipmap)
	{
		const int32 CellsCount = FieldInfos.ClipmapCount * EFieldPhysicsType::Field_PhysicsType_Max;
		{
			RHI_BREADCRUMB_EVENT_STAT(RHICmdList, PhysicsFields_ClearClipmap, "PhysicsFields_ClearClipmap");

			RHICmdList.Transition(FRHITransitionInfo(this->ClipmapBuffer.UAV, ERHIAccess::Unknown, ERHIAccess::UAVCompute));

			TShaderMapRef<FResetPhysicsFieldClipmapCS> ComputeShader(GetGlobalShaderMap(GetFeatureLevel()));
			SetComputePipelineState(RHICmdList, ComputeShader.GetComputeShader());

			const uint32 NumCells = FieldInfos.CellsOffsets[CellsCount];
			const uint32 NumGroups = FMath::DivideAndRoundUp<int32>(NumCells, FResetPhysicsFieldClipmapCS::ThreadGroupSize);

			SetShaderParametersLegacyCS(RHICmdList, ComputeShader, this, TimeSeconds, NumCells);
			DispatchComputeShader(RHICmdList, ComputeShader.GetShader(), NumGroups, 1, 1);
			UnsetShaderParametersLegacyCS(RHICmdList, ComputeShader);

			RHICmdList.Transition(FRHITransitionInfo(this->ClipmapBuffer.UAV, ERHIAccess::Unknown, ERHIAccess::SRVCompute));
		}
		{
			RHI_BREADCRUMB_EVENT_STAT(RHICmdList, PhysicsFields_UpdateBuffers, "PhysicsFields_UpdateBuffers");

			FieldInfos.ClipmapCenter = FieldInfos.ViewOrigin;
			UpdateBounds(TargetsMinDatas, TargetsMaxDatas, TargetsOffsetsDatas, BoundsOffsetsDatas);

			UpdateInternalBuffer<int32, 1, EPixelFormat::PF_R32_SINT>(RHICmdList, TEXT("PhysicsField_CellsOffsets"), FieldInfos.CellsOffsets.Num(), FieldInfos.CellsOffsets.GetData(), CellsOffsets);
			UpdateInternalBuffer<FIntVector4, 1, EPixelFormat::PF_R32G32B32A32_UINT>(RHICmdList, TEXT("PhysicsField_CellsMin"), FieldInfos.CellsMin.Num(), FieldInfos.CellsMin.GetData(), CellsMin);
			UpdateInternalBuffer<FIntVector4, 1, EPixelFormat::PF_R32G32B32A32_UINT>(RHICmdList, TEXT("PhysicsField_CellsMax"), FieldInfos.CellsMax.Num(), FieldInfos.CellsMax.GetData(), CellsMax);
		}
		{
			RHI_BREADCRUMB_EVENT_STAT(RHICmdList, PhysicsFields_BuildClipmap, "PhysicsFields_BuildClipmap");

			RHICmdList.Transition(FRHITransitionInfo(this->ClipmapBuffer.UAV, ERHIAccess::Unknown, ERHIAccess::UAVCompute));

			TShaderMapRef<FBuildPhysicsFieldClipmapCS> ComputeShader(GetGlobalShaderMap(GetFeatureLevel()));
			SetComputePipelineState(RHICmdList, ComputeShader.GetComputeShader());

			const uint32 NumCells = FieldInfos.CellsOffsets[CellsCount];
			const uint32 NumGroups = FMath::DivideAndRoundUp<int32>(NumCells, FBuildPhysicsFieldClipmapCS::ThreadGroupSize);

			SetShaderParametersLegacyCS(RHICmdList, ComputeShader, this, TimeSeconds, NumCells);
			DispatchComputeShader(RHICmdList, ComputeShader.GetShader(), NumGroups, 1, 1);
			UnsetShaderParametersLegacyCS(RHICmdList, ComputeShader);

			RHICmdList.Transition(FRHITransitionInfo(this->ClipmapBuffer.UAV, ERHIAccess::UAVCompute, ERHIAccess::SRVMask));
		}
	}
}

/**
*	FPhysicsFieldInstance
*/

void FPhysicsFieldInstance::InitInstance( const TArray<EFieldPhysicsType>& TargetTypes, const bool bBuildClipmap, const bool bNeedsResource)
{
	FPhysicsFieldInfos::BufferOffsets VectorTargets(InPlace, -1), ScalarTargets(InPlace, -1), IntegerTargets(InPlace, -1), PhysicsTargets(InPlace, -1);
	TStaticArray<UE::Core::TAlignedElement<FIntVector4, 16>, MAX_PHYSICS_FIELD_TARGETS> PhysicsBounds(InPlace, FIntVector4(-1));

	static const TArray<EFieldPhysicsType> VectorTypes = GetFieldTargetTypes(EFieldOutputType::Field_Output_Vector);
	static const TArray<EFieldPhysicsType> ScalarTypes = GetFieldTargetTypes(EFieldOutputType::Field_Output_Scalar);
	static const TArray<EFieldPhysicsType> IntegerTypes = GetFieldTargetTypes(EFieldOutputType::Field_Output_Integer);

	int32 TargetIndex = 0;
	int32 TargetCount = 0;
	for (auto& TargetType : TargetTypes)
	{
		PhysicsTargets[TargetType] = TargetCount;
		const EFieldOutputType OutputType = GetFieldTargetIndex(VectorTypes, ScalarTypes, IntegerTypes, TargetType, TargetIndex);
		if (OutputType == EFieldOutputType::Field_Output_Vector)
		{
			VectorTargets[TargetIndex] = TargetCount;
			PhysicsBounds[TargetIndex].X = TargetType;
			TargetCount += 3;
		}
		else if (OutputType == EFieldOutputType::Field_Output_Scalar)
		{
			ScalarTargets[TargetIndex] = TargetCount;
			PhysicsBounds[TargetIndex].Y = TargetType;
			TargetCount += 1;
		}
		else if (OutputType == EFieldOutputType::Field_Output_Integer)
		{
			IntegerTargets[TargetIndex] = TargetCount;
			PhysicsBounds[TargetIndex].Z = TargetType;
			TargetCount += 1;
		}
	}
	
	if (!FieldResource && bNeedsResource)
	{
		FieldResource = new FPhysicsFieldResource(TargetCount, TargetTypes, VectorTargets, ScalarTargets, IntegerTargets, PhysicsTargets, PhysicsBounds, bBuildClipmap);

		FPhysicsFieldResource* LocalFieldResource = FieldResource;
		ENQUEUE_RENDER_COMMAND(FInitPhysicsFieldResourceCommand)(
			[LocalFieldResource](FRHICommandList& RHICmdList)
			{
				LocalFieldResource->InitResource(RHICmdList);
			});
	}
}

void FPhysicsFieldInstance::ReleaseInstance()
{
	if (FieldResource)
	{
		FPhysicsFieldResource* LocalFieldResource = FieldResource;
		ENQUEUE_RENDER_COMMAND(FDestroyPhysicsFieldResourceCommand)(
			[LocalFieldResource] (FRHICommandListBase&)
		{
			LocalFieldResource->ReleaseResource();
			delete LocalFieldResource;
		});
		FieldResource = nullptr;
	}

	NodesOffsets.Empty();
	NodesParams.Empty();
	TargetsMax.Empty();
	TargetsMin.Empty();
	BoundsMin.Empty();
	BoundsMax.Empty();
}

void FPhysicsFieldInstance::UpdateInstance(const float TimeSeconds, const bool bIsDebugBuffer)
{
	NodesOffsets.Reset();
	NodesParams.Reset();

	BoundsMin.Reset();
	BoundsMax.Reset();

	if (FieldResource)
	{
		TargetsMin.Init(FVector(0), EFieldPhysicsType::Field_PhysicsType_Max);
		TargetsMax.Init(FVector(0), EFieldPhysicsType::Field_PhysicsType_Max);
		
		for (auto& TargetOffset : TargetsOffsets)
		{
			TargetOffset = 0;
		}
		for (auto& BoundsOffset : BoundsOffsets)
		{
			BoundsOffset = 0;
		}
		for (auto& TargetType : FieldResource->FieldInfos.TargetTypes)
		{
			const int32 NodeOffset = NodesOffsets.Num();
			FVector TargetMin(FLT_MAX), TargetMax(-FLT_MAX);
			int32 NumCommands = 0;
			
			const EFieldOutputType OutputType = GetFieldTargetOutput(TargetType);
			for (auto& FieldCommand : FieldCommands)
			{
				const EFieldPhysicsType CommandType = GetFieldPhysicsType(FieldCommand.TargetAttribute);
				if (CommandType == TargetType)
				{
					UPhysicsFieldComponent::BuildCommandBounds(FieldCommand);


					FVector MinBounds = FieldCommand.BoundingBox.Min;
					FVector MaxBounds = FieldCommand.BoundingBox.Max;
					if (bIsDebugBuffer && ((MinBounds.X <= -FLT_MAX) || (MaxBounds.X >= FLT_MAX)))
					{
						// When no bounds default size is 1m for the box
						MinBounds = FieldCommand.CenterPosition - FVector(50, 50, 50);
						MaxBounds = FieldCommand.CenterPosition + FVector(50, 50, 50);
					}
					BoundsMin.Add(FVector4(MinBounds, 0.0));
					BoundsMax.Add(FVector4(MaxBounds, FieldCommand.MaxMagnitude));

					TargetMin = MinVector(TargetMin, FieldCommand.BoundingBox.Min);
					TargetMax = MaxVector(TargetMax, FieldCommand.BoundingBox.Max);
					
					// The order should N0,C0,N1,C1,S1,N2,C2,S2...(N: node data, C: conversion data, S: Sum data)
					// Loop over the instances directly instead of building an instance graph to avoid stack overflow
					if(NumCommands == 1)
					{
						// Pre fill datas : Only append a conversion field (int->float) for the first command 
						// to be used by the SumScalar in the post fill
						if (OutputType == EFieldOutputType::Field_Output_Integer)
						{
							FConversionField<int32, float>::FillDefaultSetup(NodesOffsets, NodesParams);
						}
					}

					FieldCommand.RootNode->FillSetupDatas(NodesOffsets, NodesParams, FieldCommand.TimeCreation);

					if(NumCommands > 0)
					{
						// Post fill datas : Append a sum vector field in between the previous node and the current one
						if (OutputType == EFieldOutputType::Field_Output_Vector)
						{
							FSumVector::FillDefaultSetup(NodesOffsets, NodesParams);
						}
						// Post fill datas : Append a sum scalar field in between the previous node and the current one
						else if (OutputType == EFieldOutputType::Field_Output_Scalar)
						{
							FSumScalar::FillDefaultSetup(NodesOffsets, NodesParams);
						}
						// Post fill datas : Append a conversion field (int->float) + sum scalar field in between the previous node and the current one
						else if (OutputType == EFieldOutputType::Field_Output_Integer)
						{
							FConversionField<int32, float>::FillDefaultSetup(NodesOffsets, NodesParams);
							FSumScalar::FillDefaultSetup(NodesOffsets, NodesParams);
						}
					}
					++NumCommands;
				}
			}
			BoundsOffsets[TargetType+1] = NumCommands;
			TargetsOffsets[TargetType+1] = NodesOffsets.Num() - NodeOffset;
			
			TargetsMin[TargetType] = (NumCommands == 0) ? FVector(-FLT_MAX) : TargetMin;
			TargetsMax[TargetType] = (NumCommands == 0) ? FVector(FLT_MAX) : TargetMax;
		}
		
		for (uint32 FieldIndex = 1; FieldIndex < EFieldPhysicsType::Field_PhysicsType_Max + 1; ++FieldIndex)
		{
			TargetsOffsets[FieldIndex] += TargetsOffsets[FieldIndex - 1];
			BoundsOffsets[FieldIndex] += BoundsOffsets[FieldIndex - 1];
		}
		{
			TStaticArray<int32, EFieldPhysicsType::Field_PhysicsType_Max + 1> LocalTargetsOffsets = TargetsOffsets;
			TArray<int32> LocalNodesOffsets = NodesOffsets;
			TArray<float> LocalNodesParams = NodesParams;

			TArray<FVector> LocalTargetsMin = TargetsMin;
			TArray<FVector> LocalTargetsMax = TargetsMax;

			TStaticArray<int32, EFieldPhysicsType::Field_PhysicsType_Max + 1> LocalBoundsOffsets = BoundsOffsets;
			TArray<FVector4> LocalBoundsMin = BoundsMin;
			TArray<FVector4> LocalBoundsMax = BoundsMax;

			const float LocalTimeSeconds = TimeSeconds;

			FPhysicsFieldResource* LocalFieldResource = FieldResource;
			ENQUEUE_RENDER_COMMAND(FUpdateFieldInstanceCommand)(
				[LocalFieldResource, LocalNodesParams, LocalNodesOffsets, LocalTargetsOffsets, LocalTargetsMin, LocalTargetsMax, LocalTimeSeconds, LocalBoundsOffsets, LocalBoundsMin, LocalBoundsMax](FRHICommandList& RHICmdList)
			{
				LocalFieldResource->UpdateResource(RHICmdList, 
				LocalTargetsOffsets, LocalNodesOffsets, LocalNodesParams, LocalTargetsMin, LocalTargetsMax, LocalTimeSeconds, LocalBoundsMin, LocalBoundsMax, LocalBoundsOffsets);
			});
		}
	}
}


/**
*	PhysicsFieldComponent
*/

UPhysicsFieldComponent::UPhysicsFieldComponent()
{
	PrimaryComponentTick.bCanEverTick = true;
	PrimaryComponentTick.bStartWithTickEnabled = true;
}

void UPhysicsFieldComponent::CreateRenderState_Concurrent(FRegisterComponentContext* Context)
{
	SCOPE_CYCLE_COUNTER(STAT_PhysicsFields_CreateSceneProxy);

	Super::CreateRenderState_Concurrent(Context);

	if (!FieldProxy)
	{
		FieldProxy = new FPhysicsFieldSceneProxy(this);
	}

	if (FieldProxy && GetWorld() && GetWorld()->Scene)
	{
		GetWorld()->Scene->SetPhysicsField(FieldProxy);
	}
}

void UPhysicsFieldComponent::DestroyRenderState_Concurrent()
{
	Super::DestroyRenderState_Concurrent();

	if (FieldProxy && GetWorld() && GetWorld()->Scene)
	{
		GetWorld()->Scene->ResetPhysicsField();

		FPhysicsFieldSceneProxy* SceneProxy = FieldProxy;
		ENQUEUE_RENDER_COMMAND(FDestroySkyLightCommand)(
			[SceneProxy](FRHICommandList& RHICmdList)
		{
			delete SceneProxy;
		});

		FieldProxy = nullptr;
	}
}

void UPhysicsFieldComponent::SendRenderDynamicData_Concurrent()
{
	SCOPE_CYCLE_COUNTER(STAT_PhysicsFields_SendRenderData);

	Super::SendRenderDynamicData_Concurrent();

	static IConsoleVariable* TransientLifetimeCVar = IConsoleManager::Get().FindConsoleVariable(TEXT("r.PhysicsField.Rendering.TransientLifetime"));
	const float TransientLifetime = TransientLifetimeCVar ? TransientLifetimeCVar->GetFloat() : 0.0f;

	TArray<uint8> FieldBuffers = { (uint8)(EFieldCommandBuffer::GPUFieldBuffer) ,(uint8)(EFieldCommandBuffer::GPUDebugBuffer) };
	for (uint32 FieldIndex = 0; FieldIndex < 2; ++FieldIndex)
	{
		FPhysicsFieldInstance*& LocalInstance = (FieldIndex == 0) ? FieldInstance : DebugInstance;
		if (LocalInstance)
		{
			const bool bPreviousUpdate = LocalInstance->FieldCommands.Num() > 0;
			const bool bIsDebugBuffer = FieldIndex == 1;

			const float TimeSeconds = GetWorld() ? GetWorld()->TimeSeconds : 0.0;

			LocalInstance->FieldCommands.Empty();
			LocalInstance->FieldCommands.Append(PersistentCommands[FieldBuffers[FieldIndex]]);
			LocalInstance->FieldCommands.Append(TransientCommands[FieldBuffers[FieldIndex]]);
			LocalInstance->FieldCommands.Append(ConstructionCommands[FieldBuffers[FieldIndex]]);
			
			if (bIsDebugBuffer && TransientLifetime > 0.0f)
			{
				for (int32 CommandIndex = TransientCommands[FieldBuffers[FieldIndex]].Num() - 1; CommandIndex >= 0; --CommandIndex)
				{
					if (TimeSeconds - TransientCommands[FieldBuffers[FieldIndex]][CommandIndex].TimeCreation > TransientLifetime)
					{
						TransientCommands[FieldBuffers[FieldIndex]].RemoveAt(CommandIndex);
					}
				}
			}
			else
			{
				TransientCommands[FieldBuffers[FieldIndex]].Empty();
			}

			const bool bCurrentUpdate = LocalInstance->FieldCommands.Num() > 0;

			if (bCurrentUpdate || bPreviousUpdate)
			{
				LocalInstance->UpdateInstance(TimeSeconds, bIsDebugBuffer);
			}
		}
	}
}

void UPhysicsFieldComponent::OnRegister()
{
	Super::OnRegister();
	
	bool bNeedsResource = false;
	if (GetWorld() && GetWorld()->Scene )
	{
		const EShaderPlatform ShaderPlatfom = GetWorld()->Scene->GetShaderPlatform();
		bNeedsResource = !IsMobilePlatform(ShaderPlatfom) && IsFeatureLevelSupported(ShaderPlatfom, ERHIFeatureLevel::SM5);
	}

	TArray<bool> bBuildClipmaps = { GPhysicsFieldBuildClipmap == 1, false };
	for (uint32 FieldIndex = 0; FieldIndex < 2; ++FieldIndex)
	{
		FPhysicsFieldInstance*& LocalInstance = (FieldIndex == 0) ? FieldInstance : DebugInstance;
		if (!LocalInstance)
		{
			LocalInstance = new FPhysicsFieldInstance();

			if (GPhysicsFieldSingleTarget == 1)
			{
				TArray<EFieldPhysicsType> TargetTypes = { EFieldPhysicsType::Field_LinearForce };
				LocalInstance->InitInstance(TargetTypes, bBuildClipmaps[FieldIndex], bNeedsResource);
			}
			else
			{
				TArray<EFieldPhysicsType> TargetTypes = { EFieldPhysicsType::Field_DynamicState,
															EFieldPhysicsType::Field_LinearForce,
															EFieldPhysicsType::Field_ExternalClusterStrain,
															EFieldPhysicsType::Field_Kill,
															EFieldPhysicsType::Field_LinearVelocity,
															EFieldPhysicsType::Field_AngularVelociy,
															EFieldPhysicsType::Field_AngularTorque,
															EFieldPhysicsType::Field_InternalClusterStrain,
															EFieldPhysicsType::Field_DisableThreshold,
															EFieldPhysicsType::Field_SleepingThreshold,
															EFieldPhysicsType::Field_CollisionGroup,
															EFieldPhysicsType::Field_ActivateDisabled };
				TargetTypes.Sort();

				LocalInstance->InitInstance(TargetTypes, bBuildClipmaps[FieldIndex], bNeedsResource);
			}
		}
	}
}

void UPhysicsFieldComponent::OnUnregister()
{
	for (uint32 FieldIndex = 0; FieldIndex < 2; ++FieldIndex)
	{
		FPhysicsFieldInstance*& LocalInstance = (FieldIndex == 0) ? FieldInstance : DebugInstance;
		if (LocalInstance)
		{
			LocalInstance->ReleaseInstance();

			FPhysicsFieldInstance* LocalFieldInstance = LocalInstance;
			ENQUEUE_RENDER_COMMAND(FDestroyPhysicsFieldInstanceCommand)(
				[LocalFieldInstance](FRHICommandList& RHICmdList)
				{
					delete LocalFieldInstance;
				});
			LocalInstance = nullptr;
		}
	}
	Super::OnUnregister();
}

void UPhysicsFieldComponent::TickComponent(float DeltaTime, enum ELevelTick TickType, FActorComponentTickFunction* ThisTickFunction)
{
	Super::TickComponent(DeltaTime, TickType, ThisTickFunction);

	// Swap the read and write chaos fields list. That way we have a consistent result independently of the ticking group of the component using it
	PersistentCommands[(uint8)(EFieldCommandBuffer::CPUReadBuffer)] = PersistentCommands[(uint8)(EFieldCommandBuffer::CPUWriteBuffer)];
	TransientCommands[(uint8)(EFieldCommandBuffer::CPUReadBuffer)] = TransientCommands[(uint8)(EFieldCommandBuffer::CPUWriteBuffer)];

	// Transient commands have a tick life time to be able to be fetched from the chaos components
	// Persistent ones will be destrtoyed on request by the field component itslef through the RemovePersistentCommand method
	TransientCommands[(uint8)(EFieldCommandBuffer::CPUWriteBuffer)].Reset();

	MarkRenderDynamicDataDirty();
}

void UPhysicsFieldComponent::BuildCommandBounds(FFieldSystemCommand& FieldCommand)
{
	FieldCommand.BoundingBox.Min = FVector(-FLT_MAX);
	FieldCommand.BoundingBox.Max = FVector(FLT_MAX);
	FieldCommand.MaxMagnitude = 1.0;
	FieldCommand.CenterPosition = FVector::Zero();

	if (FieldCommand.RootNode.IsValid())
	{
		FieldCommand.MaxMagnitude = FieldCommand.RootNode->EvalMaxMagnitude();
		FieldCommand.RootNode->ComputeFieldBounds(FieldCommand.BoundingBox.Min, FieldCommand.BoundingBox.Max, FieldCommand.CenterPosition);
	}
}

void UPhysicsFieldComponent::AddTransientCommand(const FFieldSystemCommand& FieldCommand, const bool bIsWorldField)
{
	if (bIsWorldField)
	{
		TransientCommands[(uint8)(EFieldCommandBuffer::GPUFieldBuffer)].Add(FieldCommand);
	}
	else
	{
		TransientCommands[(uint8)(EFieldCommandBuffer::CPUWriteBuffer)].Add(FieldCommand);
		TransientCommands[(uint8)(EFieldCommandBuffer::GPUDebugBuffer)].Add(FieldCommand);
	}
}

void UPhysicsFieldComponent::AddPersistentCommand(const FFieldSystemCommand& FieldCommand, const bool bIsWorldField)
{
	if (bIsWorldField)
	{
		PersistentCommands[(uint8)(EFieldCommandBuffer::GPUFieldBuffer)].Add(FieldCommand);
	}
	else
	{
		PersistentCommands[(uint8)(EFieldCommandBuffer::CPUWriteBuffer)].Add(FieldCommand);
		PersistentCommands[(uint8)(EFieldCommandBuffer::GPUDebugBuffer)].Add(FieldCommand);
	}
}

void UPhysicsFieldComponent::AddConstructionCommand(const FFieldSystemCommand& FieldCommand)
{
	ConstructionCommands[(uint8)(EFieldCommandBuffer::GPUDebugBuffer)].Add(FieldCommand);
}

void UPhysicsFieldComponent::RemoveTransientCommand(const FFieldSystemCommand& FieldCommand, const bool bIsWorldField)
{
	if (bIsWorldField)
	{
		TransientCommands[(uint8)(EFieldCommandBuffer::GPUFieldBuffer)].Remove(FieldCommand);
	}
	else
	{
		TransientCommands[(uint8)(EFieldCommandBuffer::CPUWriteBuffer)].Remove(FieldCommand);
		TransientCommands[(uint8)(EFieldCommandBuffer::GPUDebugBuffer)].Remove(FieldCommand);
	}
}

void UPhysicsFieldComponent::RemovePersistentCommand(const FFieldSystemCommand& FieldCommand, const bool bIsWorldField)
{
	if (bIsWorldField)
	{
		PersistentCommands[(uint8)(EFieldCommandBuffer::GPUFieldBuffer)].Remove(FieldCommand);
	}
	else
	{
		PersistentCommands[(uint8)(EFieldCommandBuffer::CPUWriteBuffer)].Remove(FieldCommand);
		PersistentCommands[(uint8)(EFieldCommandBuffer::GPUDebugBuffer)].Remove(FieldCommand);
	}
}

FORCEINLINE void FillFieldCommands(const TArray<FFieldSystemCommand>& InputCommands, const FBox& BoundingBox, const float TimeSeconds, TArray<FFieldSystemCommand>& OutputCommands) 
{
	TArray<float> FieldTimes;
	FieldTimes.Init(TimeSeconds, InputCommands.Num());

	// Store the previous creation time if the command was already in the output commands list. If not set it from the input time.
	uint32 CommandIndex = 0;
	for (const FFieldSystemCommand& InputCommand : InputCommands)
	{
		const int32 OutputIndex = OutputCommands.Find(InputCommand);
		FieldTimes[CommandIndex++] = (OutputIndex != INDEX_NONE) ? OutputCommands[OutputIndex].TimeCreation : TimeSeconds;
	}

	OutputCommands.Empty();
	CommandIndex = 0;

	for (const FFieldSystemCommand& InputCommand : InputCommands)
	{
		if (InputCommand.BoundingBox.Intersect(BoundingBox))
		{
			OutputCommands.Add_GetRef(InputCommand).TimeCreation = FieldTimes[CommandIndex];
		}
		++CommandIndex;
	}
}

void UPhysicsFieldComponent::FillTransientCommands(const bool bIsWorldField, const FBox& BoundingBox, const float TimeSeconds, TArray<FFieldSystemCommand>& OutputCommands) const
{
	const TArray<FFieldSystemCommand>& InputCommands = bIsWorldField ? 
		TransientCommands[(uint8)(EFieldCommandBuffer::GPUFieldBuffer)] : TransientCommands[(uint8)(EFieldCommandBuffer::CPUReadBuffer)];

	FillFieldCommands(InputCommands, BoundingBox, TimeSeconds, OutputCommands);
}

void UPhysicsFieldComponent::FillPersistentCommands(const bool bIsWorldField, const FBox& BoundingBox, const float TimeSeconds, TArray<FFieldSystemCommand>& OutputCommands) const
{
	const TArray<FFieldSystemCommand>& InputCommands = bIsWorldField ?
		PersistentCommands[(uint8)(EFieldCommandBuffer::GPUFieldBuffer)] : PersistentCommands[(uint8)(EFieldCommandBuffer::CPUReadBuffer)];

	FillFieldCommands(InputCommands, BoundingBox, TimeSeconds, OutputCommands);
}

void GetFieldIndex(const uint32 FieldType, int32& FieldIndex, EFieldOutputType& FieldOutput)
{
	const EFieldPhysicsType FieldPhysicsType = GetFieldPhysicsTypes()[FieldType];

	int32 TargetIndex = INDEX_NONE;
	FieldOutput = GetFieldTargetIndex(GetFieldTargetTypes(EFieldOutputType::Field_Output_Vector),
		GetFieldTargetTypes(EFieldOutputType::Field_Output_Scalar),
		GetFieldTargetTypes(EFieldOutputType::Field_Output_Integer), FieldPhysicsType, FieldIndex);
}

/**
 * FPhysicsFieldSceneProxy.
 */

FPhysicsFieldSceneProxy::FPhysicsFieldSceneProxy(UPhysicsFieldComponent* PhysicsFieldComponent)
{
	if (PhysicsFieldComponent && PhysicsFieldComponent->FieldInstance)
	{
		FieldResource = PhysicsFieldComponent->FieldInstance->FieldResource;
	}
	if (PhysicsFieldComponent && PhysicsFieldComponent->DebugInstance)
	{
		DebugResource = PhysicsFieldComponent->DebugInstance->FieldResource;
	}
}

FPhysicsFieldSceneProxy::~FPhysicsFieldSceneProxy()
{}

struct FVectorFieldAddOperator
{
	static void BlendValues(const FVector& VectorValueA, const FVector& VectorValueB, FVector& VectorValueC)
	{
		VectorValueC = VectorValueA + VectorValueB;
	}
};

struct FScalarFieldAddOperator
{
	static void BlendValues(const float& VectorValueA, const float& VectorValueB, float& VectorValueC)
	{
		VectorValueC = VectorValueA + VectorValueB;
	}
};

struct FIntegerFieldAddOperator
{
	static void BlendValues(const int32& VectorValueA, const int32& VectorValueB, int32& VectorValueC)
	{
		VectorValueC = VectorValueA + VectorValueB;
	}
};

template<typename DataType, typename BlendOperator>
void EvaluateFieldNodes(TArray<FFieldSystemCommand>& FieldCommands, const EFieldPhysicsType FieldType, FFieldContext& FieldContext, 
	TArray<DataType>& ResultsArray, TArray<DataType>& MaxArray)
{
	bool HasMatchingCommand = false;
	if (FieldCommands.Num() > 0 && ResultsArray.Num() == MaxArray.Num())
	{
		SCOPE_CYCLE_COUNTER(STAT_NiagaraUpdateField_Object);
		TFieldArrayView<DataType> ResultsView(ResultsArray, 0, ResultsArray.Num());

		const float TimeSeconds = FieldContext.TimeSeconds;
		for (int32 CommandIndex = 0; CommandIndex < FieldCommands.Num(); ++CommandIndex)
		{
			const FName AttributeName = FieldCommands[CommandIndex].TargetAttribute;
			FieldContext.TimeSeconds = TimeSeconds - FieldCommands[CommandIndex].TimeCreation;

			const EFieldPhysicsType CommandType = GetFieldPhysicsType(AttributeName);
			if (CommandType == FieldType && FieldCommands[CommandIndex].RootNode.Get())
			{
				FFieldNode<DataType>* RootNode = static_cast<FFieldNode<DataType>*>(
					FieldCommands[CommandIndex].RootNode.Get());

				RootNode->Evaluate(FieldContext, ResultsView);
				HasMatchingCommand = true;

				for (int32 InstanceIdx = 0; InstanceIdx < MaxArray.Num(); ++InstanceIdx)
				{
					// TODO : First version with the add. will probably have to include an operator as a template argument 
					BlendOperator::BlendValues(MaxArray[InstanceIdx], ResultsArray[InstanceIdx], MaxArray[InstanceIdx]);
				} 
			}
		}
	}
	if (!HasMatchingCommand)
	{
		MaxArray.Init(DataType(0), ResultsArray.Num());
	}
}

void EvaluateFieldVectorNodes(TArray<FFieldSystemCommand>& FieldCommands, const EFieldPhysicsType FieldType, FFieldContext& FieldContext, 
	TArray<FVector>& ResultsArray, TArray<FVector>& MaxArray)
{
	EvaluateFieldNodes<FVector, FVectorFieldAddOperator>(FieldCommands,FieldType,FieldContext,ResultsArray,MaxArray);
}

void EvaluateFieldScalarNodes(TArray<FFieldSystemCommand>& FieldCommands, const EFieldPhysicsType FieldType, FFieldContext& FieldContext, 
	TArray<float>& ResultsArray, TArray<float>& MaxArray)
{
	EvaluateFieldNodes<float, FScalarFieldAddOperator>(FieldCommands,FieldType,FieldContext,ResultsArray,MaxArray);
}

void EvaluateFieldIntegerNodes(TArray<FFieldSystemCommand>& FieldCommands, const EFieldPhysicsType FieldType, FFieldContext& FieldContext, 
	TArray<int32>& ResultsArray, TArray<int32>& MaxArray)
{
	EvaluateFieldNodes<int32, FIntegerFieldAddOperator>(FieldCommands,FieldType,FieldContext,ResultsArray,MaxArray);
}

UPhysicsFieldStatics::UPhysicsFieldStatics(class FObjectInitializer const & ObjectInitializer)
	: Super(ObjectInitializer)
{
}

FVector UPhysicsFieldStatics::EvalPhysicsVectorField(const UObject* WorldContextObject, const FVector& WorldPosition, const EFieldVectorType TargetType)
{
	if(UWorld* ThisWorld = GEngine->GetWorldFromContextObject(WorldContextObject, EGetWorldErrorMode::LogAndReturnNull))
	{
		FFieldExecutionDatas ExecutionDatas;
		ExecutionDatas.SamplePositions.Init(WorldPosition, 1);
		ExecutionDatas.SampleIndices.Init(FFieldContextIndex(0,0),1);
		
		TArray<FVector>& SampleResults = ExecutionDatas.VectorResults[(uint8)EFieldCommandResultType::FinalResult];
		SampleResults.Init(FVector::ZeroVector, 1);

		TArray<FVector> SampleMax;
		SampleMax.Init(FVector::ZeroVector, 1);

		FFieldContext FieldContext{
			ExecutionDatas,
			FFieldContext::UniquePointerMap(),
			ThisWorld->TimeSeconds
		};

		const EFieldPhysicsType PhysicsType = GetFieldTargetTypes(Field_Output_Vector)[TargetType];
		UPhysicsFieldComponent* FieldComponent = ThisWorld->PhysicsField;
		if (FieldComponent && FieldComponent->FieldInstance)
		{
			EvaluateFieldVectorNodes(FieldComponent->FieldInstance->FieldCommands, PhysicsType, FieldContext, SampleResults, SampleMax);
			return SampleMax[0];
		}
	}
	return FVector::Zero();
}

float UPhysicsFieldStatics::EvalPhysicsScalarField(const UObject* WorldContextObject, const FVector& WorldPosition, const EFieldScalarType TargetType)
{
	if(UWorld* ThisWorld = GEngine->GetWorldFromContextObject(WorldContextObject, EGetWorldErrorMode::LogAndReturnNull))
	{
		FFieldExecutionDatas ExecutionDatas;
		ExecutionDatas.SamplePositions.Init(WorldPosition, 1);
		ExecutionDatas.SampleIndices.Init(FFieldContextIndex(0,0),1);
		
		TArray<float>& SampleResults = ExecutionDatas.ScalarResults[(uint8)EFieldCommandResultType::FinalResult];
		SampleResults.Init(0.0f, 1);

		TArray<float> SampleMax;
		SampleMax.Init(0.0f, 1);

		FFieldContext FieldContext{
			ExecutionDatas,
			FFieldContext::UniquePointerMap(),
			ThisWorld->TimeSeconds
		};

		const EFieldPhysicsType PhysicsType = GetFieldTargetTypes(Field_Output_Scalar)[TargetType];
		UPhysicsFieldComponent* FieldComponent = ThisWorld->PhysicsField;
		if (FieldComponent && FieldComponent->FieldInstance)
		{
			EvaluateFieldScalarNodes(FieldComponent->FieldInstance->FieldCommands, PhysicsType, FieldContext, SampleResults, SampleMax);
			return SampleMax[0];
		}
	}
	return 0.0;
}

int32 UPhysicsFieldStatics::EvalPhysicsIntegerField(const UObject* WorldContextObject, const FVector& WorldPosition, const EFieldIntegerType TargetType)
{
	if(UWorld* ThisWorld = GEngine->GetWorldFromContextObject(WorldContextObject, EGetWorldErrorMode::LogAndReturnNull))
	{
		FFieldExecutionDatas ExecutionDatas;
		ExecutionDatas.SamplePositions.Init(WorldPosition, 1);
		ExecutionDatas.SampleIndices.Init(FFieldContextIndex(0,0),1);
		
		TArray<int32>& SampleResults = ExecutionDatas.IntegerResults[(uint8)EFieldCommandResultType::FinalResult];
		SampleResults.Init(0, 1);

		TArray<int32> SampleMax;
		SampleMax.Init(0, 1);

		FFieldContext FieldContext{
			ExecutionDatas,
			FFieldContext::UniquePointerMap(),
			ThisWorld->TimeSeconds
		};

		const EFieldPhysicsType PhysicsType = GetFieldTargetTypes(Field_Output_Scalar)[TargetType];
		UPhysicsFieldComponent* FieldComponent = ThisWorld->PhysicsField;
		if (FieldComponent && FieldComponent->FieldInstance)
		{
			EvaluateFieldIntegerNodes(FieldComponent->FieldInstance->FieldCommands, PhysicsType, FieldContext, SampleResults, SampleMax);
			return SampleMax[0];
		}
	}
	return 0;
}






