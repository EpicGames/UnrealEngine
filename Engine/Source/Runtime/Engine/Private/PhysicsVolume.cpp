// Copyright Epic Games, Inc. All Rights Reserved.

#include "GameFramework/PhysicsVolume.h"
#include "Engine/World.h"
#include "Components/BrushComponent.h"
#include "PhysicsEngine/PhysicsSettings.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PhysicsVolume)

APhysicsVolume::APhysicsVolume(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	static FName CollisionProfileName("OverlapAllDynamic");
	GetBrushComponent()->SetCollisionProfileName(CollisionProfileName);

	FluidFriction = UPhysicsSettings::Get()->DefaultFluidFriction;
	TerminalVelocity = UPhysicsSettings::Get()->DefaultTerminalVelocity;
	bAlwaysRelevant = true;
	SetNetUpdateFrequency(0.1f);
	SetReplicatingMovement(false);
}

#if WITH_EDITOR
void APhysicsVolume::LoadedFromAnotherClass(const FName& OldClassName)
{
	Super::LoadedFromAnotherClass(OldClassName);

	if(GetLinkerUEVersion() < VER_UE4_REMOVE_DYNAMIC_VOLUME_CLASSES)
	{
		static FName DynamicPhysicsVolume_NAME("DynamicPhysicsVolume");

		if(OldClassName == DynamicPhysicsVolume_NAME)
		{
			GetBrushComponent()->Mobility = EComponentMobility::Movable;
		}
	}
}
#endif

void APhysicsVolume::PostInitializeComponents()
{
	Super::PostInitializeComponents();
	GetWorld()->AddPhysicsVolume(this);
}

void APhysicsVolume::Destroyed()
{
	UWorld* MyWorld = GetWorld();
	if (MyWorld)
	{
		MyWorld->RemovePhysicsVolume(this);
	}
	Super::Destroyed();
}

void APhysicsVolume::EndPlay(const EEndPlayReason::Type EndPlayReason)
{
	UWorld* MyWorld = GetWorld();
	if (MyWorld)
	{
		MyWorld->RemovePhysicsVolume(this);

		if ((EndPlayReason == EEndPlayReason::RemovedFromWorld) || (EndPlayReason == EEndPlayReason::Destroyed))
		{
			// Prevent UpdatePhysicsVolume() calls below from returning this component.
			UPrimitiveComponent* VolumeBrushComponent = GetBrushComponent();
			const bool bSavedGenerateOverlapEvents = VolumeBrushComponent->GetGenerateOverlapEvents();
			VolumeBrushComponent->SetGenerateOverlapEvents(false);

			// Refresh physics volume on any components touching this volume.
			// TODO: Physics volume tracking code needs a cleanup, ideally it just uses normal begin/end overlap events,
			// but there are subtleties with the stacking and priority rules.
			const TArray<FOverlapInfo>& Overlaps = VolumeBrushComponent->GetOverlapInfos();
			for (const FOverlapInfo& Info : Overlaps)
			{
				UPrimitiveComponent* OtherPrim = Info.OverlapInfo.GetComponent();
				if (OtherPrim && OtherPrim->GetShouldUpdatePhysicsVolume())
				{
					OtherPrim->UpdatePhysicsVolume(true);
				}
			}

			// Restore saved flag, since we may stream back in.
			VolumeBrushComponent->SetGenerateOverlapEvents(bSavedGenerateOverlapEvents);
		}
	}
	Super::EndPlay(EndPlayReason);
}


bool APhysicsVolume::IsOverlapInVolume(const class USceneComponent& TestComponent) const
{
	bool bInsideVolume = true;
	if (!bPhysicsOnContact)
	{
		FVector ClosestPoint(0.f);
		// If there is no primitive component as root we consider you inside the volume. This is odd, but the behavior 
		// has existed for a long time, so keeping it this way
		UPrimitiveComponent* RootPrimitive = Cast<UPrimitiveComponent>(GetRootComponent());
		if (RootPrimitive)
		{
			float DistToCollisionSqr = -1.f;
			if (RootPrimitive->GetSquaredDistanceToCollision(TestComponent.GetComponentLocation(), DistToCollisionSqr, ClosestPoint))
			{
				bInsideVolume = (DistToCollisionSqr == 0.f);
			}
			else
			{
				bInsideVolume = false;
			}
		}
	}

	return bInsideVolume;
}

float APhysicsVolume::GetGravityZ() const
{
	const UWorld* MyWorld = GetWorld();
	return MyWorld ? MyWorld->GetGravityZ() : UPhysicsSettings::Get()->DefaultGravityZ;
}

void APhysicsVolume::ActorEnteredVolume(AActor* Other) {}

void APhysicsVolume::ActorLeavingVolume(AActor* Other) {}


