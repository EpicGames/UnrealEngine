// Copyright Epic Games, Inc. All Rights Reserved.

#include "PieFixupSerializer.h"

#include "Components/InstancedStaticMeshComponent.h"
#include "Misc/AutomationTest.h"
#include "UObject/Package.h"
#include "UObject/UnrealType.h"

class FMulticastDelegateProperty;

namespace
{
	void DefaultSoftObjectPathFixupFunction(int32, FSoftObjectPath&) {}
}

FPIEFixupSerializer::FPIEFixupSerializer(UObject* InRoot, int32 InPIEInstanceID)
	: FPIEFixupSerializer(InRoot, InPIEInstanceID, DefaultSoftObjectPathFixupFunction)
{
}

FPIEFixupSerializer::FPIEFixupSerializer(UObject* InRoot, int32 InPIEInstanceID, TFunctionRef<void(int32, FSoftObjectPath&)> InSoftObjectPathFixupFunction)
	: SoftObjectPathFixupFunction(InSoftObjectPathFixupFunction)
	, Root(InRoot)
	, PIEInstanceID(InPIEInstanceID)
{
	this->ArShouldSkipBulkData = true;
	this->ArIsObjectReferenceCollector = true;
	this->ArIsModifyingWeakAndStrongReferences = true;

	// Don't trigger serialization of compilable assets
	SetShouldSkipCompilingAssets(true);
}

bool FPIEFixupSerializer::ShouldSkipProperty(const FProperty* InProperty) const
{
	return InProperty->IsA<FMulticastDelegateProperty>() || FArchiveUObject::ShouldSkipProperty(InProperty);
}

FArchive& FPIEFixupSerializer::operator<<(UObject*& Object)
{
	if (Object && (Object == Root ||Object->IsIn(Root)) && !VisitedObjects.Contains(Object))
	{
		VisitedObjects.Add(Object);

#if WITH_EDITOR
		if (UPackage* ExternalPackage = Object->GetExternalPackage())
		{
			checkf(Object->IsPackageExternal(), TEXT("Expected an external package. Package: '%s'. Object: '%s'."), *ExternalPackage->GetFullName(), *Object->GetFullName());
			checkf(ExternalPackage->HasAnyPackageFlags(PKG_PlayInEditor), TEXT("Package missing the PKG_PlayInEditor flag. Package: '%s'. Object: '%s'."), *ExternalPackage->GetFullName(), *Object->GetFullName());
			ExternalPackage->SetPIEInstanceID(PIEInstanceID);
		}
#endif

		// Skip instanced static mesh component as their impact on serialization is enormous and they don't contain lazy ptrs.
		if (!Cast<UInstancedStaticMeshComponent>(Object))
		{
			if (RecursionCount > 0)
			{
				DeferredSerialize.Add(Object);
			}
			else
			{
				RecursionCount++;
				Object->Serialize(*this);
				RecursionCount--;
			}
		}
	}

	if (RecursionCount == 0)
	{
		RecursionCount++;
		while (DeferredSerialize.Num())
		{
			DeferredSerialize.Pop(EAllowShrinking::No)->Serialize(*this);
		}
		RecursionCount--;
	}

	return *this;
}

FArchive& FPIEFixupSerializer::operator<<(FSoftObjectPath& Value)
{
	Value.FixupForPIE(PIEInstanceID, SoftObjectPathFixupFunction);
	return *this;
}

FArchive& FPIEFixupSerializer::operator<<(FLazyObjectPtr& Value)
{
	Value.FixupForPIE(PIEInstanceID);
	return *this;
}

FArchive& FPIEFixupSerializer::operator<<(FSoftObjectPtr& Value)
{
	// Forward the serialization to the FSoftObjectPath overload so it can be fixed up
	*this << Value.GetUniqueID();
	return *this;
}


#if WITH_DEV_AUTOMATION_TESTS && WITH_EDITOR

#include "Tests/PieFixupTestObjects.h"

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FPIEFixupSerializerTest, "System.Engine.PIE.FixupSoftReferences", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::EngineFilter);
bool FPIEFixupSerializerTest::RunTest(const FString& Parameters)
{
	FName PackageName = MakeUniqueObjectName(nullptr, UPackage::StaticClass(), FName("/Memory/PieFixupTest"));
	UPackage* Package = NewObject<UPackage>(nullptr, UPackage::StaticClass(), PackageName);
	UPieFixupTestObject* Obj = NewObject<UPieFixupTestObject>(Package);
	Obj->Path = TEXT("/Game/Maps/Arena.Arena:PersistentLevel.SpawnPoint.Root");
	Obj->TypedPtr = TSoftObjectPtr<AActor>(FSoftObjectPath(TEXT("/Game/Maps/Arena.Arena:PersistentLevel.Target")));
	
	Obj->Struct.Path = Obj->Path;
	Obj->Struct.TypedPtr = Obj->TypedPtr;
	
	FPieFixupStructWithSoftObjectPath& InArray = Obj->Array.AddZeroed_GetRef();
	InArray.Path = Obj->Path;
	InArray.TypedPtr = Obj->TypedPtr;
	
	int32 PieInstanceID  = 3;
	FPIEFixupSerializer Fixup(Package, PieInstanceID );
	Fixup << Obj;
	
	TestEqual(TEXT("Path is fixed up"), Obj->Path.ToString(),
		FString::Printf(TEXT("/Game/Maps/%s_%d_Arena.Arena:PersistentLevel.SpawnPoint.Root"), PLAYWORLD_PACKAGE_PREFIX, PieInstanceID));
	TestEqual(TEXT("Ptr is fixed up"), Obj->TypedPtr.GetUniqueID().ToString(),
		FString::Printf(TEXT("/Game/Maps/%s_%d_Arena.Arena:PersistentLevel.Target"), PLAYWORLD_PACKAGE_PREFIX, PieInstanceID));
	TestEqual(TEXT("Path in struct is fixed up"), Obj->Struct.Path.ToString(),
		FString::Printf(TEXT("/Game/Maps/%s_%d_Arena.Arena:PersistentLevel.SpawnPoint.Root"), PLAYWORLD_PACKAGE_PREFIX, PieInstanceID));
	TestEqual(TEXT("Ptr in struct is fixed up"), Obj->Struct.TypedPtr.GetUniqueID().ToString(),
		FString::Printf(TEXT("/Game/Maps/%s_%d_Arena.Arena:PersistentLevel.Target"), PLAYWORLD_PACKAGE_PREFIX, PieInstanceID));
	TestEqual(TEXT("Path in array of structs is fixed up "), InArray.Path.ToString(),
		FString::Printf(TEXT("/Game/Maps/%s_%d_Arena.Arena:PersistentLevel.SpawnPoint.Root"), PLAYWORLD_PACKAGE_PREFIX, PieInstanceID));
	TestEqual(TEXT("Ptr in array of structs is fixed up "), InArray.TypedPtr.GetUniqueID().ToString(),
		FString::Printf(TEXT("/Game/Maps/%s_%d_Arena.Arena:PersistentLevel.Target"), PLAYWORLD_PACKAGE_PREFIX, PieInstanceID));

	return true;
}

#endif // WITH_DEV_AUTOMATION_TESTS && WITH_EDITOR