// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Set.h"
#include "Serialization/ArchiveUObject.h"

class FPIEFixupSerializer : public FArchiveUObject
{
public:
	FPIEFixupSerializer(UObject* InRoot, int32 InPIEInstanceID);
	FPIEFixupSerializer(UObject* InRoot, int32 InPIEInstanceID, TFunctionRef<void(int32, FSoftObjectPath&)> InSoftObjectPathFixupFunction);

	bool ShouldSkipProperty(const FProperty* InProperty) const override;

	FArchive& operator<<(UObject*& Object) override;
	FArchive& operator<<(FSoftObjectPath& Value) override;
	FArchive& operator<<(FLazyObjectPtr& LazyObjectPtr) override;
	FArchive& operator<<(FSoftObjectPtr& Value) override;

private:
	TFunctionRef<void(int32, FSoftObjectPath&)> SoftObjectPathFixupFunction;
	TSet<UObject*> VisitedObjects;
	TArray<UObject*, TInlineAllocator<32>> DeferredSerialize;
	int32 RecursionCount = 0;
	UObject* Root;
	int32 PIEInstanceID;
};