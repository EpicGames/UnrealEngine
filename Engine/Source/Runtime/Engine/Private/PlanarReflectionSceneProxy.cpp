// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	PlanarReflectionProxy.cpp
=============================================================================*/

#include "PlanarReflectionSceneProxy.h"
#include "GameFramework/Actor.h"
#include "Components/PlanarReflectionComponent.h"

FPlanarReflectionSceneProxy::FPlanarReflectionSceneProxy(UPlanarReflectionComponent* Component) :
	bIsStereo(false), RenderTarget(nullptr)
{
	float ClampedFadeStart = FMath::Max(Component->DistanceFromPlaneFadeoutStart, 0.0f);
	float ClampedFadeEnd = FMath::Max(Component->DistanceFromPlaneFadeoutEnd, 0.0f);

	DistanceFromPlaneFadeEnd = ClampedFadeEnd;

	float DistanceFadeScale = 1.0f / FMath::Max(ClampedFadeEnd - ClampedFadeStart, UE_DELTA);

	PlanarReflectionParameters = FVector(
		DistanceFadeScale,
		-ClampedFadeStart * DistanceFadeScale,
		Component->NormalDistortionStrength);

	const float CosFadeStart = FMath::Cos(FMath::Clamp(Component->AngleFromPlaneFadeStart, 0.1f, 89.9f) * (float)UE_PI / 180.0f);
	const float CosFadeEnd = FMath::Cos(FMath::Clamp(Component->AngleFromPlaneFadeEnd, 0.1f, 89.9f) * (float)UE_PI / 180.0f);
	const float Range = 1.0f / FMath::Max(CosFadeStart - CosFadeEnd, UE_DELTA);

	PlanarReflectionParameters2 = FVector2D(
		Range,
		-CosFadeEnd * Range);

	Component->GetProjectionWithExtraFOV(ProjectionWithExtraFOV[0], 0);
	Component->GetProjectionWithExtraFOV(ProjectionWithExtraFOV[1], 1);

	ViewRect[0] = FIntRect(-1, -1, -1, -1);
	ViewRect[1] = FIntRect(-1, -1, -1, -1);

	OwnerName = Component->GetOwner() ? Component->GetOwner()->GetFName() : NAME_None;

	UpdateTransform(Component->GetComponentTransform().ToMatrixWithScale());

	PlanarReflectionId = Component->GetPlanarReflectionId();
	PrefilterRoughness = Component->PrefilterRoughness;
	PrefilterRoughnessDistance = Component->PrefilterRoughnessDistance;
}
