// Copyright Epic Games, Inc. All Rights Reserved.

#include "PlatformFeatures.h"
#include "SaveGameSystem.h"
#include "DVRStreaming.h"
#include "VideoRecordingSystem.h"

#if PLATFORM_ANDROID
extern bool AndroidThunkCpp_IsScreenCaptureDisabled();
extern void AndroidThunkCpp_DisableScreenCapture(bool bDisable);
#endif

ISaveGameSystem* IPlatformFeaturesModule::GetSaveGameSystem()
{
	static FGenericSaveGameSystem GenericSaveGame;
	return &GenericSaveGame;
}


IDVRStreamingSystem* IPlatformFeaturesModule::GetStreamingSystem()
{
	static FGenericDVRStreamingSystem GenericStreamingSystem;
	return &GenericStreamingSystem;
}

FString IPlatformFeaturesModule::GetUniqueAppId()
{
	return FString();
}

IVideoRecordingSystem* IPlatformFeaturesModule::GetVideoRecordingSystem()
{
	static FGenericVideoRecordingSystem GenericVideoRecordingSystem;
	return &GenericVideoRecordingSystem;
}

void IPlatformFeaturesModule::SetScreenshotEnableState(bool bEnabled)
{
#if PLATFORM_ANDROID
	if (AndroidThunkCpp_IsScreenCaptureDisabled() != !bEnabled)
	{
		AndroidThunkCpp_DisableScreenCapture(!bEnabled);
	}
#endif
}

static FAutoConsoleCommand CVarPFMSetScreenshotEnableState(
	TEXT("pf.SetScreenshotEnableState"),
	TEXT("Enables or disables taking screenshots if the platform supports it."),
	FConsoleCommandWithArgsDelegate::CreateLambda([](const TArray<FString>& Args)
	{
		if (Args.Num() > 0)
		{
			IPlatformFeaturesModule::Get().SetScreenshotEnableState(Args[0].ToBool());
		}
	})
);

static FAutoConsoleCommand CVarPFMEnableRecording(
	TEXT("pf.EnableRecording"),
	TEXT("Enables or disables recording if the platform supports it."),
	FConsoleCommandWithArgsDelegate::CreateLambda([](const TArray<FString>& Args)
	{
		IVideoRecordingSystem* VideoRecordingSystem = IPlatformFeaturesModule::Get().GetVideoRecordingSystem();
		if (VideoRecordingSystem && (Args.Num() > 0))
		{
			VideoRecordingSystem->EnableRecording(Args[0].ToBool());
		}
	})
);

static FAutoConsoleCommand CVarPFMEnableStreaming(
	TEXT("pf.EnableStreaming"),
	TEXT("Enables or disables streaming if the platform supports it."),
	FConsoleCommandWithArgsDelegate::CreateLambda([](const TArray<FString>& Args)
	{
		IDVRStreamingSystem* DVRStreamingSystem = IPlatformFeaturesModule::Get().GetStreamingSystem();
		if (DVRStreamingSystem && (Args.Num() > 0))
		{
			DVRStreamingSystem->EnableStreaming(Args[0].ToBool());
		}
	})
);