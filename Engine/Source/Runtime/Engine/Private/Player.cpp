// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	Player.cpp: Unreal player implementation.
=============================================================================*/
 
#include "Engine/Player.h"
#include "Engine/Engine.h"
#include "Engine/GameViewportClient.h"
#include "Engine/LocalPlayer.h"
#include "Engine/NetConnection.h"
#include "EngineUtils.h"

#include "GameFramework/HUD.h"
#include "GameFramework/PlayerInput.h"

#include "GameFramework/CheatManager.h"
#include "GameFramework/GameStateBase.h"

#include "AutoRTFM.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(Player)

//////////////////////////////////////////////////////////////////////////
// UPlayer

UPlayer::UPlayer(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

FString UPlayer::ConsoleCommand(const FString& Cmd, bool bWriteToLog)
{
	UNetConnection* NetConn = Cast<UNetConnection>(this);
	bool bIsBeacon = NetConn && NetConn->OwningActor && !PlayerController;

	UConsole* ViewportConsole = (GEngine->GameViewport != nullptr) ? GEngine->GameViewport->ViewportConsole : nullptr;
	FConsoleOutputDevice StrOut(ViewportConsole);
	FOutputDevice& EffectiveOutputDevice = (!bWriteToLog || (ViewportConsole != nullptr)) ? StrOut : (FOutputDevice&)(*GLog);

	const int32 CmdLen = Cmd.Len();
	TCHAR* CommandBuffer = (TCHAR*)FMemory::Malloc((CmdLen + 1)*sizeof(TCHAR));
	TCHAR* Line = (TCHAR*)FMemory::Malloc((CmdLen + 1)*sizeof(TCHAR));

	const TCHAR* Command = CommandBuffer;
	// copy the command into a modifiable buffer
	FCString::Strncpy(CommandBuffer, *Cmd.Left(CmdLen), (CmdLen + 1));

	// iterate over the line, breaking up on |'s
	while (FParse::Line(&Command, Line, CmdLen + 1))	// The FParse::Line function expects the full array size, including the NULL character.
	{
		// if dissociated with the PC, stop processing commands
		if (bIsBeacon || PlayerController)
		{
			if (!Exec(GetWorld(), Line, EffectiveOutputDevice))
			{
				StrOut.Logf(TEXT("Command not recognized: %s"), Line);
			}
		}
	}

	// Free temp arrays
	FMemory::Free(CommandBuffer);
	CommandBuffer = nullptr;

	FMemory::Free(Line);
	Line = nullptr;

	if (!bWriteToLog)
	{
		return MoveTemp(StrOut);
	}

	return TEXT("");
}

APlayerController* UPlayer::GetPlayerController(const UWorld* const InWorld) const
{
	if (InWorld == nullptr)
	{
		return PlayerController;
	}

	for (FConstPlayerControllerIterator Iterator = InWorld->GetPlayerControllerIterator(); Iterator; ++Iterator)
	{
		APlayerController* PC = Iterator->Get();
		if (PC && PC->GetLocalPlayer() == this)
		{
			return PC;
		}
	}

	return nullptr;
}

#if UE_ALLOW_EXEC_COMMANDS
bool UPlayer::Exec( UWorld* InWorld, const TCHAR* Cmd,FOutputDevice& Ar)
{
	// Route through Exec_Dev and Exec_Editor first
	if (FExec::Exec(InWorld, Cmd, Ar))
	{
		return true;
	}

	AActor* ExecActor = PlayerController;
	if (!ExecActor)
	{
		UNetConnection* NetConn = Cast<UNetConnection>(this);
		ExecActor = (NetConn && NetConn->OwningActor) ? ToRawPtr(NetConn->OwningActor) : nullptr;
	}

	if (ExecActor)
	{
		// Since UGameViewportClient calls Exec on UWorld, we only need to explicitly
		// call UWorld::Exec if we either have a null GEngine or a null ViewportClient
		UWorld* World = ExecActor->GetWorld();
		check(World);
		check(InWorld == nullptr || InWorld == World);
		const bool bWorldNeedsExec = GEngine == nullptr || Cast<ULocalPlayer>(this) == nullptr || static_cast<ULocalPlayer*>(this)->ViewportClient == nullptr;
		APawn* PCPawn = PlayerController ? PlayerController->GetPawnOrSpectator() : nullptr;
		if (bWorldNeedsExec && World->Exec(World, Cmd, Ar))
		{
			return true;
		}
		else if (PlayerController && PlayerController->PlayerInput && PlayerController->PlayerInput->ProcessConsoleExec(Cmd, Ar, PCPawn))
		{
			return true;
		}
		else if (ExecActor->ProcessConsoleExec(Cmd, Ar, PCPawn))
		{
			return true;
		}
		else if (PCPawn && PCPawn->ProcessConsoleExec(Cmd, Ar, PCPawn))
		{
			return true;
		}
		else if (PlayerController && PlayerController->MyHUD && PlayerController->MyHUD->ProcessConsoleExec(Cmd, Ar, PCPawn))
		{
			return true;
		}
		else if (World->GetAuthGameMode() && World->GetAuthGameMode()->ProcessConsoleExec(Cmd, Ar, PCPawn))
		{
			return true;
		}
		else if (PlayerController && PlayerController->CheatManager && PlayerController->CheatManager->ProcessConsoleExec(Cmd, Ar, PCPawn))
		{
			return true;
		}
		else if (World->GetGameState() && World->GetGameState()->ProcessConsoleExec(Cmd, Ar, PCPawn))
		{
			return true;
		}
		else if (PlayerController && PlayerController->PlayerCameraManager && PlayerController->PlayerCameraManager->ProcessConsoleExec(Cmd, Ar, PCPawn))
		{
			return true;
		}
	}
	return false;
}
#endif // UE_ALLOW_EXEC_COMMANDS

void UPlayer::SwitchController(class APlayerController* PC)
{
	// Detach old player.
	if( this->PlayerController )
	{
		this->PlayerController->Player = NULL;
	}

	// Set the viewport.
	PC->Player = this;
	this->PlayerController = PC;
	
	ReceivedPlayerController(PC);
}

void UPlayer::ReceivedPlayerController(class APlayerController* NewController)
{
}
