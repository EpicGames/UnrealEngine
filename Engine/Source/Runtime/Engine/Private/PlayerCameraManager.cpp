// Copyright Epic Games, Inc. All Rights Reserved.

#include "Camera/PlayerCameraManager.h"
#include "GameFramework/Pawn.h"
#include "Engine/Engine.h"
#include "Engine/HitResult.h"
#include "Engine/World.h"
#include "Camera/CameraActor.h"
#include "Engine/Canvas.h"
#include "Features/IModularFeatures.h"
#include "GameFramework/PlayerController.h"
#include "GameFramework/WorldSettings.h"
#include "AudioDevice.h"
#include "Particles/EmitterCameraLensEffectBase.h"
#include "Camera/CameraComponent.h"
#include "Camera/CameraModifier_CameraShake.h"
#include "Camera/CameraModularFeature.h"
#include "Camera/CameraPhotography.h"
#include "Camera/CameraShakeBase.h"
#include "GameFramework/PlayerState.h"
#include "IXRTrackingSystem.h" // for IsHeadTrackingAllowed()
#include "GameFramework/GameNetworkManager.h"
#include "TimerManager.h"
#include "Camera/CameraLensEffectInterface.h"
#include "GameDelegates.h"
#include "ChaosVisualDebugger/ChaosVisualDebuggerTrace.h"

#if UE_WITH_REMOTE_OBJECT_HANDLE
#include "UObject/RemoteObject.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(PlayerCameraManager)

DEFINE_LOG_CATEGORY_STATIC(LogPlayerCameraManager, Log, All);

DECLARE_CYCLE_STAT(TEXT("ServerUpdateCamera"), STAT_ServerUpdateCamera, STATGROUP_Game);
DECLARE_CYCLE_STAT(TEXT("Camera ProcessViewRotation"), STAT_Camera_ProcessViewRotation, STATGROUP_Game);


//////////////////////////////////////////////////////////////////////////
// APlayerCameraManager

APlayerCameraManager::APlayerCameraManager(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	DefaultFOV = 90.0f;
	DefaultAspectRatio = 1.33333f;
	bDefaultConstrainAspectRatio = false;
	DefaultOrthoWidth = 512.0f;
	bAutoCalculateOrthoPlanes = true;
	AutoPlaneShift = 0.0f;
	bUpdateOrthoPlanes = true;
	bUseCameraHeightAsViewTarget = true;
	SetHidden(true);
	bReplicates = false;
	FreeCamDistance = 256.0f;
	bDebugClientSideCamera = false;
	ViewPitchMin = -89.9f;
	ViewPitchMax = 89.9f;
	ViewYawMin = 0.f;
	ViewYawMax = 359.999f;
	ViewRollMin = -89.9f;
	ViewRollMax = 89.9f;
	bUseClientSideCameraUpdates = true;
	CameraStyle = NAME_Default;
	SetCanBeDamaged(false);
	TimeSinceLastServerUpdateCamera = 0.0f;

	// create dummy transform component
	TransformComponent = CreateDefaultSubobject<USceneComponent>(TEXT("TransformComponent0"));
	RootComponent = TransformComponent;

	// support camerashakes by default
	DefaultModifiers.Add(UCameraModifier_CameraShake::StaticClass());
}

/// @cond DOXYGEN_WARNINGS

void APlayerCameraManager::PhotographyCameraModify_Implementation(const FVector NewCameraLocation, const FVector PreviousCameraLocation, const FVector OriginalCameraLocation, FVector& OutCameraLocation)
{	
	FCameraPhotographyManager::Get().DefaultConstrainCamera(NewCameraLocation, PreviousCameraLocation, OriginalCameraLocation, OutCameraLocation, this);
}

void APlayerCameraManager::OnPhotographySessionStart_Implementation()
{	// do nothing by default
}

void APlayerCameraManager::OnPhotographySessionEnd_Implementation()
{	// do nothing by default
}

void APlayerCameraManager::OnPhotographyMultiPartCaptureStart_Implementation()
{	// do nothing by default
}

void APlayerCameraManager::OnPhotographyMultiPartCaptureEnd_Implementation()
{	// do nothing by default
}

/// @endcond


APlayerController* APlayerCameraManager::GetOwningPlayerController() const
{
	return PCOwner;
}

void APlayerCameraManager::SwapPendingViewTargetWhenUsingClientSideCameraUpdates()
{
	if (PendingViewTarget.Target)
	{
		AssignViewTarget(PendingViewTarget.Target, ViewTarget);
		ViewTarget.CheckViewTarget(PCOwner);
		// remove old pending ViewTarget so we don't still try to switch to it
		PendingViewTarget.Target = NULL;
	}
}

void APlayerCameraManager::SetViewTarget(class AActor* NewTarget, struct FViewTargetTransitionParams TransitionParams)
{
	// Make sure view target is valid
	if (NewTarget == NULL)
	{
		NewTarget = PCOwner;
	}

	// Update current ViewTargets
	ViewTarget.CheckViewTarget(PCOwner);
	if (PendingViewTarget.Target)
	{
		PendingViewTarget.CheckViewTarget(PCOwner);
	}

	// If we're already transitioning to this new target, don't interrupt.
	if (PendingViewTarget.Target != NULL && NewTarget == PendingViewTarget.Target)
	{
		return;
	}

	if (UWorld* World = GetWorld())
	{
		World->GetTimerManager().ClearTimer(SwapPendingViewTargetWhenUsingClientSideCameraUpdatesTimerHandle);
	}

	// if viewtarget different then new one or we're transitioning from the same target with locked outgoing, then assign it
	if ((NewTarget != ViewTarget.Target) || (PendingViewTarget.Target && BlendParams.bLockOutgoing))
	{
		// if a transition time is specified, then set pending view target accordingly
		if (TransitionParams.BlendTime > 0)
		{
			// band-aid fix so that EndViewTarget() gets called properly in this case
			if (PendingViewTarget.Target == NULL)
			{
				PendingViewTarget.Target = ViewTarget.Target;
			}

			// use last frame's POV
			ViewTarget.POV = GetLastFrameCameraCacheView();
			BlendTimeToGo = TransitionParams.BlendTime;

			AssignViewTarget(NewTarget, PendingViewTarget, TransitionParams);
			PendingViewTarget.CheckViewTarget(PCOwner);

			const ENetMode CurNetMode = GetNetMode();
			if (bUseClientSideCameraUpdates && (CurNetMode != NM_Client && CurNetMode != NM_Standalone))
			{
				if (UWorld* World = GetWorld())
				{
					World->GetTimerManager().SetTimer(SwapPendingViewTargetWhenUsingClientSideCameraUpdatesTimerHandle, this, &ThisClass::SwapPendingViewTargetWhenUsingClientSideCameraUpdates, TransitionParams.BlendTime, false);
				}
			}
		}
		else
		{
			// otherwise, assign new viewtarget instantly
			AssignViewTarget(NewTarget, ViewTarget);
			ViewTarget.CheckViewTarget(PCOwner);
			// remove old pending ViewTarget so we don't still try to switch to it
			PendingViewTarget.Target = NULL;
		}
	}
	else
	{
		// we're setting the viewtarget to the viewtarget we were transitioning away from,
		// just abort the transition.
		// @fixme, investigate if we want this case to go through the above code, so AssignViewTarget et al
		// get called
		if (PendingViewTarget.Target != NULL)
		{
			if (!PCOwner->IsPendingKillPending() && !PCOwner->IsLocalPlayerController() && GetNetMode() != NM_Client)
			{
				PCOwner->ClientSetViewTarget(NewTarget, TransitionParams);
			}
		}
		PendingViewTarget.Target = NULL;
	}

	// update the blend params after all the assignment logic so that sub-classes can compare
	// the old vs new parameters if needed.
	BlendParams = TransitionParams;
}


void APlayerCameraManager::AssignViewTarget(AActor* NewTarget, FTViewTarget& VT, struct FViewTargetTransitionParams TransitionParams)
{
	if (!NewTarget)
	{
		return;
	}

	// Skip assigning to the same target unless we have a pending view target that's bLockOutgoing
	if (NewTarget == VT.Target && !(PendingViewTarget.Target && BlendParams.bLockOutgoing))
	{
		return;
	}

// 	UE_LOGF(LogPlayerCameraManager, Log, "%f AssignViewTarget OldTarget: %ls, NewTarget: %ls, BlendTime: %f", GetWorld()->TimeSeconds, VT.Target ? *VT.Target->GetFName().ToString() : TEXT("NULL"),
// 		NewTarget ? *NewTarget->GetFName().ToString() : TEXT("NULL"),
// 		TransitionParams.BlendTime);

	AActor* OldViewTarget = VT.Target;
	VT.Target = NewTarget;

	// Use default FOV and aspect ratio.
	VT.POV.AspectRatio = DefaultAspectRatio;
	VT.POV.bConstrainAspectRatio = bDefaultConstrainAspectRatio;
	VT.POV.FOV = DefaultFOV;

	if (OldViewTarget)
	{
		OldViewTarget->EndViewTarget(PCOwner);
	}

	VT.Target->BecomeViewTarget(PCOwner);

	if (!PCOwner->IsLocalPlayerController() && (GetNetMode() != NM_Client))
	{
		PCOwner->ClientSetViewTarget(VT.Target, TransitionParams);
	}

	FGameDelegates::Get().GetViewTargetChangedDelegate().Broadcast(PCOwner, OldViewTarget, NewTarget);
}

AActor* APlayerCameraManager::GetViewTarget() const
{
	// to handle verification/caching behavior while preserving constness upstream
	APlayerCameraManager* const NonConstThis = const_cast<APlayerCameraManager*>(this);

	// if blending to another view target, return this one first
	if( PendingViewTarget.Target )
	{
		NonConstThis->PendingViewTarget.CheckViewTarget(NonConstThis->PCOwner);
		if( PendingViewTarget.Target )
		{
			return PendingViewTarget.Target;
		}
	}

	NonConstThis->ViewTarget.CheckViewTarget(NonConstThis->PCOwner);
	return ViewTarget.Target;
}

APawn* APlayerCameraManager::GetViewTargetPawn() const
{
	// to handle verification/caching behavior while preserving constness upstream
	APlayerCameraManager* const NonConstThis = const_cast<APlayerCameraManager*>(this);

	// if blending to another view target, return this one first
	if (PendingViewTarget.Target)
	{
		NonConstThis->PendingViewTarget.CheckViewTarget(NonConstThis->PCOwner);
		if (PendingViewTarget.Target)
		{
			return PendingViewTarget.GetTargetPawn();
		}
	}

	NonConstThis->ViewTarget.CheckViewTarget(NonConstThis->PCOwner);
	return ViewTarget.GetTargetPawn();
}

bool APlayerCameraManager::ShouldTickIfViewportsOnly() const
{
	return (PCOwner != NULL);
}

void APlayerCameraManager::ApplyCameraModifiers(float DeltaTime, FMinimalViewInfo& InOutPOV)
{
	ClearCachedPPBlends();

	// Loop through each camera modifier
	ForEachCameraModifier([DeltaTime, &InOutPOV](UCameraModifier* CameraModifier)
	{
		bool bContinue = true;

		// Apply camera modification and output into DesiredCameraOffset/DesiredCameraRotation
		if ((CameraModifier != NULL) && !CameraModifier->IsDisabled())
		{
			// If ModifyCamera returns true, exit loop
			// Allows high priority things to dictate if they are
			// the last modifier to be applied
			bContinue = !CameraModifier->ModifyCamera(DeltaTime, InOutPOV);
		}

		return bContinue;
	});
}

void APlayerCameraManager::AddCachedPPBlend(struct FPostProcessSettings& PPSettings, float BlendWeight, EViewTargetBlendOrder BlendOrder)
{
	check(PostProcessBlendCache.Num() == PostProcessBlendCacheWeights.Num());
	check(PostProcessBlendCache.Num() == PostProcessBlendCacheOrders.Num());
	PostProcessBlendCache.Add(PPSettings);
	PostProcessBlendCacheWeights.Add(BlendWeight);
	PostProcessBlendCacheOrders.Add(BlendOrder);
}

void APlayerCameraManager::ClearCachedPPBlends()
{
	PostProcessBlendCache.Empty();
	PostProcessBlendCacheWeights.Empty();
	PostProcessBlendCacheOrders.Empty();
}

void APlayerCameraManager::GetCachedPostProcessBlends(TArray<FPostProcessSettings> const*& OutPPSettings, TArray<float> const*& OutBlendWeights) const
{
	OutPPSettings = &PostProcessBlendCache;
	OutBlendWeights = &PostProcessBlendCacheWeights;
}

void APlayerCameraManager::GetCachedPostProcessBlends(TArray<FPostProcessSettings> const*& OutPPSettings, TArray<float> const*& OutBlendWeights, TArray<EViewTargetBlendOrder> const*& OutBlendOrders) const
{
	OutPPSettings = &PostProcessBlendCache;
	OutBlendWeights = &PostProcessBlendCacheWeights;
	OutBlendOrders = &PostProcessBlendCacheOrders;
}

void APlayerCameraManager::UpdateViewTargetInternal(FTViewTarget& OutVT, float DeltaTime)
{
	if (OutVT.Target)
	{
		FVector OutLocation;
		FRotator OutRotation;
		float OutFOV;

		if (BlueprintUpdateCamera(OutVT.Target, OutLocation, OutRotation, OutFOV))
		{
			OutVT.POV.Location = OutLocation;
			OutVT.POV.Rotation = OutRotation;
			OutVT.POV.FOV = OutFOV;
		}
		else
		{
			OutVT.Target->CalcCamera(DeltaTime, OutVT.POV);
		}
	}
}

void APlayerCameraManager::UpdateViewTarget(FTViewTarget& OutVT, float DeltaTime)
{
	// Don't update outgoing viewtarget during an interpolation 
	if ((PendingViewTarget.Target != NULL) && BlendParams.bLockOutgoing && OutVT.Equal(ViewTarget))
	{
		return;
	}

	// Store previous POV, in case we need it later
	FMinimalViewInfo OrigPOV = OutVT.POV;

	// Reset the view target POV fully
	static const FMinimalViewInfo DefaultViewInfo;
	OutVT.POV = DefaultViewInfo;
	OutVT.POV.FOV = DefaultFOV;
	OutVT.POV.OrthoWidth = DefaultOrthoWidth;
	OutVT.POV.AspectRatio = DefaultAspectRatio;
	OutVT.POV.bConstrainAspectRatio = bDefaultConstrainAspectRatio;
	OutVT.POV.ProjectionMode = bIsOrthographic ? ECameraProjectionMode::Orthographic : ECameraProjectionMode::Perspective;
	OutVT.POV.PostProcessBlendWeight = 1.0f;
	OutVT.POV.bAutoCalculateOrthoPlanes = bAutoCalculateOrthoPlanes;
	OutVT.POV.AutoPlaneShift = AutoPlaneShift;
	OutVT.POV.bUpdateOrthoPlanes = bUpdateOrthoPlanes;
	OutVT.POV.bUseCameraHeightAsViewTarget = bUseCameraHeightAsViewTarget;

	bool bDoNotApplyModifiers = false;

	if (ACameraActor* CamActor = Cast<ACameraActor>(OutVT.Target))
	{
		// Viewing through a camera actor.
		CamActor->GetCameraComponent()->GetCameraView(DeltaTime, OutVT.POV);
	}
	else
	{

		static const FName NAME_Fixed = FName("Fixed");
		static const FName NAME_ThirdPerson = FName("ThirdPerson");
		static const FName NAME_FreeCam = FName("FreeCam");
		static const FName NAME_FreeCam_Default = FName("FreeCam_Default");
		static const FName NAME_FirstPerson = FName("FirstPerson");

		if (CameraStyle == NAME_Fixed)
		{
			// do not update, keep previous camera position by restoring
			// saved POV, in case CalcCamera changes it but still returns false
			OutVT.POV = OrigPOV;

			// don't apply modifiers when using this debug camera mode
			bDoNotApplyModifiers = true;
		}
		else if (CameraStyle == NAME_ThirdPerson || CameraStyle == NAME_FreeCam || CameraStyle == NAME_FreeCam_Default)
		{
			// Simple third person view implementation
			FVector Loc = OutVT.Target->GetActorLocation();
			FRotator Rotator = OutVT.Target->GetActorRotation();

			if (OutVT.Target == PCOwner)
			{
				Loc = PCOwner->GetFocalLocation();
			}

			// Take into account Mesh Translation so it takes into account the PostProcessing we do there.
			// @fixme, can crash in certain BP cases where default mesh is null
//			APawn* TPawn = Cast<APawn>(OutVT.Target);
// 			if ((TPawn != NULL) && (TPawn->Mesh != NULL))
// 			{
// 				Loc += FQuatRotationMatrix(OutVT.Target->GetActorQuat()).TransformVector(TPawn->Mesh->RelativeLocation - GetDefault<APawn>(TPawn->GetClass())->Mesh->RelativeLocation);
// 			}

			//OutVT.Target.GetActorEyesViewPoint(Loc, Rot);
			if( CameraStyle == NAME_FreeCam || CameraStyle == NAME_FreeCam_Default )
			{
				Rotator = PCOwner->GetControlRotation();
			}

			FVector Pos = Loc + ViewTargetOffset + FRotationMatrix(Rotator).TransformVector(FreeCamOffset) - Rotator.Vector() * FreeCamDistance;
			FCollisionQueryParams BoxParams(SCENE_QUERY_STAT(FreeCam), false, this);
			BoxParams.AddIgnoredActor(OutVT.Target);
			FHitResult Result;

			GetWorld()->SweepSingleByChannel(Result, Loc, Pos, FQuat::Identity, ECC_Camera, FCollisionShape::MakeBox(FVector(12.f)), BoxParams);
			OutVT.POV.Location = !Result.bBlockingHit ? Pos : Result.Location;
			OutVT.POV.Rotation = Rotator;

			// don't apply modifiers when using this debug camera mode
			bDoNotApplyModifiers = true;
		}
		else if (CameraStyle == NAME_FirstPerson)
		{
			// Simple first person, view through viewtarget's 'eyes'
			OutVT.Target->GetActorEyesViewPoint(OutVT.POV.Location, OutVT.POV.Rotation);
	
			// don't apply modifiers when using this debug camera mode
			bDoNotApplyModifiers = true;
		}
		else
		{
			UpdateViewTargetInternal(OutVT, DeltaTime);
		}
	}

	if (!bDoNotApplyModifiers || bAlwaysApplyModifiers)
	{
		// Apply camera modifiers at the end (view shakes for example)
		ApplyCameraModifiers(DeltaTime, OutVT.POV);
	}

	// Synchronize the actor with the view target results
	SetActorLocationAndRotation(OutVT.POV.Location, OutVT.POV.Rotation, false);
	if (bAutoCalculateOrthoPlanes && OutVT.Target)
	{
		OutVT.POV.SetCameraToViewTarget(OutVT.Target->GetActorLocation());
	}

	UpdateCameraLensEffects(OutVT);
}


void APlayerCameraManager::UpdateCameraLensEffects(const FTViewTarget& OutVT)
{
	for (int32 Idx=0; Idx<CameraLensEffects.Num(); ++Idx)
	{
		if (CameraLensEffects[Idx] != NULL)
		{
			CameraLensEffects[Idx]->UpdateLocation(OutVT.POV.Location, OutVT.POV.Rotation, OutVT.POV.FOV);
		}
	}
}

void APlayerCameraManager::ApplyAudioFade()
{
	// If an audio fade event has been bound, we'd like it to override the default fade behavior.
	if (OnAudioFadeChangeEvent.IsBound())
	{
		return;
	}

	if (GEngine)
	{
		UWorld* World = GetWorld();
		if (World)
		{
			if (FAudioDevice* AudioDevice = World->GetAudioDeviceRaw())
			{
				AudioDevice->SetTransientPrimaryVolume(1.0f - FadeAmount);
			}
		}
	}
}

void APlayerCameraManager::StopAudioFade()
{
	// If an audio fade event has been bound, we'd like it to override the default fade behavior.
	if (OnAudioFadeChangeEvent.IsBound())
	{
		const bool bFadeOut = false;
		OnAudioFadeChangeEvent.Broadcast(bFadeOut, 0.f);
		return;
	}

	if (GEngine)
	{
		UWorld* World = GetWorld();
		if (World)
		{
			if (FAudioDevice* AudioDevice = World->GetAudioDeviceRaw())
			{
				AudioDevice->SetTransientPrimaryVolume(1.0f);
			}
		}
	}
}

UCameraModifier* APlayerCameraManager::AddNewCameraModifier(TSubclassOf<UCameraModifier> ModifierClass)
{
	UCameraModifier* const NewMod = NewObject<UCameraModifier>(this, ModifierClass);
	if (NewMod)
	{
		if (AddCameraModifierToList(NewMod) == true)
		{
			return NewMod;
		}
	}
	
	return nullptr;
}

UCameraModifier* APlayerCameraManager::FindCameraModifierByClass(TSubclassOf<UCameraModifier> ModifierClass)
{
	for (UCameraModifier* Mod : ModifierList)
	{
		if (Mod->GetClass() == ModifierClass)
		{
			return Mod;
		}
	}

	return nullptr;
}

bool APlayerCameraManager::AddCameraModifierToList(UCameraModifier* NewModifier)
{
	if (NewModifier)
	{
		// Look through current modifier list and find slot for this priority
		int32 BestIdx = ModifierList.Num();
		for (int32 ModifierIdx = 0; ModifierIdx < ModifierList.Num(); ModifierIdx++)
		{
			UCameraModifier* const M = ModifierList[ModifierIdx];
			if (M)
			{
				if (M == NewModifier)
				{
					// already in list, just bail
					return false;
				}

				// If priority of current index has passed or equaled ours - we have the insert location
				if (NewModifier->Priority <= M->Priority)
				{
					// Disallow addition of exclusive modifier if priority is already occupied
					if (NewModifier->bExclusive && NewModifier->Priority == M->Priority)
					{
						return false;
					}

					// Update best index
					BestIdx = ModifierIdx;

					break;
				}
			}
		}

		// Insert self into best index
		ModifierList.InsertUninitialized(BestIdx, 1);
		ModifierList[BestIdx] = NewModifier;

		// Save camera
		NewModifier->AddedToCamera(this);
		return true;
	}

	return false;
}

void APlayerCameraManager::CleanUpAnimCamera(const bool bDestroy)
{
	// clean up the temp camera actor
	if (AnimCameraActor != nullptr)
	{
		if (bDestroy)
		{
			AnimCameraActor->Destroy();
		}
		AnimCameraActor->SetOwner(nullptr);
		AnimCameraActor = nullptr;
	}
}

bool APlayerCameraManager::RemoveCameraModifier(UCameraModifier* ModifierToRemove)
{
	if (ModifierToRemove)
	{
		// Loop through each modifier in camera
		for (int32 ModifierIdx = 0; ModifierIdx < ModifierList.Num(); ModifierIdx++)
		{
			// If we found ourselves, remove ourselves from the list and return
			if (ModifierList[ModifierIdx] == ModifierToRemove)
			{
				ModifierList.RemoveAt(ModifierIdx, 1);
				return true;
			}
		}
	}

	// Didn't find it in the list, nothing removed
	return false;
}


void APlayerCameraManager::PostInitializeComponents()
{
	Super::PostInitializeComponents();

 	// Setup default camera modifiers
	TArray<TSubclassOf<UCameraModifier>> AllDefaultModifiers(DefaultModifiers);
	TArray<ICameraModularFeature*> CameraModularFeatures = IModularFeatures::Get()
		.GetModularFeatureImplementations<ICameraModularFeature>(ICameraModularFeature::GetModularFeatureName());
	for (const ICameraModularFeature* CameraModularFeature : CameraModularFeatures)
	{
		if (ensure(CameraModularFeature))
		{
			CameraModularFeature->GetDefaultModifiers(AllDefaultModifiers);
		}
	}

	if (AllDefaultModifiers.Num() > 0)
	{
		for (auto ModifierClass : AllDefaultModifiers)
		{
			// empty entries are not valid here, do work only for actual classes
			if (ModifierClass)
			{
				UCameraModifier* const NewMod = AddNewCameraModifier(ModifierClass);

				// cache ref to camera shake if this is it
				UCameraModifier_CameraShake* const ShakeMod = Cast<UCameraModifier_CameraShake>(NewMod);
				if (ShakeMod)
				{
					CachedCameraShakeMod = ShakeMod;
				}
			}
		}
	}
}

void APlayerCameraManager::EndPlay(const EEndPlayReason::Type EndPlayReason)
{
	ModifierList.Empty();
	CleanUpAnimCamera(EndPlayReason == EEndPlayReason::Destroyed);
	Super::EndPlay(EndPlayReason);
}

void APlayerCameraManager::Destroyed()
{
	CleanUpAnimCamera(true);

	Super::Destroyed();
}

#if UE_WITH_REMOTE_OBJECT_HANDLE
FRemoteServerId APlayerCameraManager::GetPreferredRemoteServer() const
{
	if (!AutoRTFM::IsClosed() && PCOwner.GetResidence() == EResidence::Remote)
	{
		// We are outside of a transaction (probably push migration step) and the
		// PCOwner is already on another server - that IS its preferred server
		FRemoteServerId PCServerId = UE::RemoteObject::Handle::GetOwnerServerId(PCOwner.GetRemoteId());
		if (PCServerId.IsValid())
		{
			return PCServerId;
		}
	}
	else if (APlayerController* LocalPC = PCOwner.Get())
	{
		return LocalPC->GetPreferredRemoteServer();
	}
	
	// No PCOwner — stay where we are (don't use spatial delegate, our location is not meaningful)
	return UE::RemoteObject::Handle::GetOwnerServerId(this);
}
#endif

void APlayerCameraManager::InitializeFor(APlayerController* PC)
{
	FMinimalViewInfo DefaultFOVCache = GetCameraCacheView();
	DefaultFOVCache.FOV = DefaultFOV;
	SetCameraCachePOV(DefaultFOVCache);

	PCOwner = PC;

	SetViewTarget(PC);

	// set the level default scale
	SetDesiredColorScale(GetWorldSettings()->DefaultColorScale, 5.f);

	// Force camera update so it doesn't sit at (0,0,0) for a full tick.
	// This can have side effects with streaming.
	UpdateCamera(0.f);
}


float APlayerCameraManager::GetFOVAngle() const
{
	return (LockedFOV > 0.f) ? LockedFOV : GetCameraCacheView().FOV;
}

void APlayerCameraManager::SetFOV(float NewFOV)
{
	LockedFOV = NewFOV;
}

void APlayerCameraManager::UnlockFOV()
{
	LockedFOV = 0.f;
}

bool APlayerCameraManager::IsOrthographic() const
{
	return bIsOrthographic;
}

float APlayerCameraManager::GetOrthoWidth() const
{
	return (LockedOrthoWidth > 0.f) ? LockedOrthoWidth : DefaultOrthoWidth;
}

void APlayerCameraManager::SetOrthoWidth(float OrthoWidth)
{
	LockedOrthoWidth = OrthoWidth;
}

void APlayerCameraManager::UnlockOrthoWidth()
{
	LockedOrthoWidth = 0.f;
}

void APlayerCameraManager::GetCameraViewPoint(FVector& OutCamLoc, FRotator& OutCamRot) const
{
	const FMinimalViewInfo& CurrentPOV = GetCameraCacheView();
	OutCamLoc = CurrentPOV.Location;
	OutCamRot = CurrentPOV.Rotation;
}

FRotator APlayerCameraManager::GetCameraRotation() const
{
	return GetCameraCacheView().Rotation;
}

FVector APlayerCameraManager::GetCameraLocation() const
{
	return GetCameraCacheView().Location;
}

void APlayerCameraManager::SetDesiredColorScale(FVector NewColorScale, float InterpTime)
{
	// if color scaling is not enabled
	if (!bEnableColorScaling)
	{
		// set the default color scale
		bEnableColorScaling = true;
		ColorScale.X = 1.f;
		ColorScale.Y = 1.f;
		ColorScale.Z = 1.f;
	}

	// Don't bother interpolating if we're already scaling at the desired color
	if( NewColorScale != ColorScale )
	{
		// save the current as original
		OriginalColorScale = ColorScale;
		// set the new desired scale
		DesiredColorScale = NewColorScale;
		// set the interpolation duration/time
		ColorScaleInterpStartTime = GetWorld()->TimeSeconds;
		ColorScaleInterpDuration = InterpTime;
		// and enable color scale interpolation
		bEnableColorScaleInterp = true;
	}
}


void APlayerCameraManager::UpdateCamera(float DeltaTime)
{
	check(PCOwner != nullptr);

	if ((PCOwner->Player && PCOwner->IsLocalPlayerController()) || !bUseClientSideCameraUpdates || bDebugClientSideCamera)
	{
		DoUpdateCamera(DeltaTime);

		const float TimeDilation = FMath::Max(GetActorTimeDilation(), UE_KINDA_SMALL_NUMBER);

		TimeSinceLastServerUpdateCamera += (DeltaTime / TimeDilation);

		if (bShouldSendClientSideCameraUpdate && IsNetMode(NM_Client))
		{
			SCOPE_CYCLE_COUNTER(STAT_ServerUpdateCamera);

			const AGameNetworkManager* const GameNetworkManager = GetDefault<AGameNetworkManager>();
			const float ClientNetCamUpdateDeltaTime = GameNetworkManager->ClientNetCamUpdateDeltaTime;
			const float ClientNetCamUpdatePositionLimit = GameNetworkManager->ClientNetCamUpdatePositionLimit;

			FVector LastCachedCameraLocation;
			FRotator LastCachedCameraRotation;
			GetLastFrameCameraCacheLocationAndRotationForServer(LastCachedCameraLocation, LastCachedCameraRotation);
			FVector CachedCameraLocation;
			FRotator CachedCameraRotation;
			GetCameraCacheLocationAndRotationForServer(CachedCameraLocation, CachedCameraRotation);

			FVector ClientCameraPosition = FRepMovement::RebaseOntoZeroOrigin(CachedCameraLocation, this);
			FVector PrevClientCameraPosition = FRepMovement::RebaseOntoZeroOrigin(LastCachedCameraLocation, this);

			const bool bPositionThreshold = (ClientCameraPosition - PrevClientCameraPosition).SizeSquared() > (ClientNetCamUpdatePositionLimit * ClientNetCamUpdatePositionLimit);

			if (bPositionThreshold || (TimeSinceLastServerUpdateCamera > ClientNetCamUpdateDeltaTime))
			{
				// compress the rotation down to 4 bytes
				int32 const ShortYaw = FRotator::CompressAxisToShort(CachedCameraRotation.Yaw);
				int32 const ShortPitch = FRotator::CompressAxisToShort(CachedCameraRotation.Pitch);
				int32 const CompressedRotation = (ShortYaw << 16) | ShortPitch;

				int32 const PrevShortYaw = FRotator::CompressAxisToShort(LastCachedCameraRotation.Yaw);
				int32 const PrevShortPitch = FRotator::CompressAxisToShort(LastCachedCameraRotation.Pitch);
				int32 const PrevCompressedRotation = (PrevShortYaw << 16) | PrevShortPitch;

				if ((CompressedRotation != PrevCompressedRotation) || !ClientCameraPosition.Equals(PrevClientCameraPosition) || (TimeSinceLastServerUpdateCamera > ServerUpdateCameraTimeout))
				{
					PCOwner->ServerUpdateCamera(ClientCameraPosition, CompressedRotation);

					TimeSinceLastServerUpdateCamera = 0.0f;
				}
			}

			bShouldSendClientSideCameraUpdate = false;
		}
	}

#if WITH_CHAOS_VISUAL_DEBUGGER
	if (FChaosVisualDebuggerTrace::IsTracing())
	{
		const FMinimalViewInfo& NewInfo = GetCameraCacheView();
		CVD_TRACE_CAMERA(this, GetWorld(), GetFName(), NewInfo.Location, FQuat(NewInfo.Rotation), NewInfo.FOV);
	}
#endif
}

bool APlayerCameraManager::AllowPhotographyMode() const
{
	return true;
}

void APlayerCameraManager::DoUpdateCamera(float DeltaTime)
{
	FMinimalViewInfo NewPOV = ViewTarget.POV;

	// update color scale interpolation
	if (bEnableColorScaleInterp)
	{
		float BlendPct = FMath::Clamp((GetWorld()->TimeSeconds - ColorScaleInterpStartTime) / ColorScaleInterpDuration, 0.f, 1.0f);
		ColorScale = FMath::Lerp(OriginalColorScale, DesiredColorScale, BlendPct);
		// if we've maxed
		if (BlendPct == 1.0f)
		{
			// disable further interpolation
			bEnableColorScaleInterp = false;
		}
	}

	// Don't update outgoing viewtarget during an interpolation when bLockOutgoing is set.
	if ((PendingViewTarget.Target == NULL) || !BlendParams.bLockOutgoing)
	{
		// Update current view target
		ViewTarget.CheckViewTarget(PCOwner);
		UpdateViewTarget(ViewTarget, DeltaTime);
	}

	// our camera is now viewing there
	NewPOV = ViewTarget.POV;

	// if we have a pending view target, perform transition from one to another.
	if (PendingViewTarget.Target != NULL)
	{
		BlendTimeToGo -= DeltaTime;

		// Update pending view target
		PendingViewTarget.CheckViewTarget(PCOwner);
		UpdateViewTarget(PendingViewTarget, DeltaTime);

		// blend....
		if (BlendTimeToGo > 0)
		{
			float DurationPct = (BlendParams.BlendTime - BlendTimeToGo) / BlendParams.BlendTime;

			float BlendPct = 0.f;
			switch (BlendParams.BlendFunction)
			{
			case VTBlend_Linear:
				BlendPct = FMath::Lerp(0.f, 1.f, DurationPct);
				break;
			case VTBlend_Cubic:
				BlendPct = FMath::CubicInterp(0.f, 0.f, 1.f, 0.f, DurationPct);
				break;
			case VTBlend_EaseIn:
				BlendPct = FMath::Lerp(0.f, 1.f, FMath::Pow(DurationPct, BlendParams.BlendExp));
				break;
			case VTBlend_EaseOut:
				BlendPct = FMath::Lerp(0.f, 1.f, FMath::Pow(DurationPct, 1.f / BlendParams.BlendExp));
				break;
			case VTBlend_EaseInOut:
				BlendPct = FMath::InterpEaseInOut(0.f, 1.f, DurationPct, BlendParams.BlendExp);
				break;
			case VTBlend_PreBlended:
				BlendPct = 1.0f;
				break;
			default:
				break;
			}

			// Update pending view target blend
			NewPOV = ViewTarget.POV;
			NewPOV.BlendViewInfo(PendingViewTarget.POV, BlendPct);//@TODO: CAMERA: Make sure the sense is correct!  BlendViewTargets(ViewTarget, PendingViewTarget, BlendPct);

			// Add this pending view target's post-process settings as an override of the main view target's one,
			// since it is blending on top of it.
			const float PendingViewTargetPPWeight = PendingViewTarget.POV.PostProcessBlendWeight * BlendPct;
			if (PendingViewTargetPPWeight > 0.f)
			{
				AddCachedPPBlend(PendingViewTarget.POV.PostProcessSettings, PendingViewTargetPPWeight, VTBlendOrder_Override);
			}
		}
		else
		{
			// we're done blending, set new view target
			ViewTarget = PendingViewTarget;

			// clear pending view target
			PendingViewTarget.Target = NULL;

			BlendTimeToGo = 0;

			// our camera is now viewing there
			NewPOV = PendingViewTarget.POV;

			OnBlendComplete().Broadcast();
		}
	}

	if (bEnableFading)
	{
		if (bAutoAnimateFade)
		{
			FadeTimeRemaining = FMath::Max(FadeTimeRemaining - DeltaTime, 0.0f);
			if (FadeTime > 0.0f)
			{
				FadeAmount = FadeAlpha.X + ((1.f - FadeTimeRemaining / FadeTime) * (FadeAlpha.Y - FadeAlpha.X));
			}

			if ((bHoldFadeWhenFinished == false) && (FadeTimeRemaining <= 0.f))
			{
				// done
				StopCameraFade();
			}
		}

		if (bFadeAudio)
		{
			ApplyAudioFade();
		}
	}

	if (AllowPhotographyMode())
	{
		const bool bPhotographyCausedCameraCut = UpdatePhotographyCamera(NewPOV);
		bGameCameraCutThisFrame = bGameCameraCutThisFrame || bPhotographyCausedCameraCut;
	}

	// Cache results
	FillCameraCache(NewPOV);
}

void APlayerCameraManager::UpdateCameraPhotographyOnly()
{
	FMinimalViewInfo NewPOV = ViewTarget.POV;

	// update photography camera, if any
	bGameCameraCutThisFrame = UpdatePhotographyCamera(NewPOV);

	// Cache results
	FillCameraCache(NewPOV);
}

void APlayerCameraManager::UpdatePhotographyPostProcessing(FPostProcessSettings& InOutPostProcessingSettings)
{
	FCameraPhotographyManager::Get().UpdatePostProcessing(InOutPostProcessingSettings);
}

//! Overridable
bool APlayerCameraManager::UpdatePhotographyCamera(FMinimalViewInfo& NewPOV)
{
	// update photography camera, if any
	return FCameraPhotographyManager::Get().UpdateCamera(NewPOV, this);
}

FPOV APlayerCameraManager::BlendViewTargets(const FTViewTarget& A,const FTViewTarget& B, float Alpha)
{
	FPOV POV;
	POV.Location = FMath::Lerp(A.POV.Location, B.POV.Location, Alpha);
	POV.FOV = (A.POV.FOV +  Alpha * ( B.POV.FOV - A.POV.FOV));

	FRotator DeltaAng = (B.POV.Rotation - A.POV.Rotation).GetNormalized();
	POV.Rotation = A.POV.Rotation + Alpha * DeltaAng;

	return POV;
}



void APlayerCameraManager::FillCameraCache(const FMinimalViewInfo& NewInfo)
{
	NewInfo.Location.DiagnosticCheckNaN(TEXT("APlayerCameraManager::FillCameraCache: NewInfo.Location"));
	NewInfo.Rotation.DiagnosticCheckNaN(TEXT("APlayerCameraManager::FillCameraCache: NewInfo.Rotation"));

	// Backup last frame results.
	const float CurrentCacheTime = GetCameraCacheTime();
	const float CurrentGameTime = GetWorld()->TimeSeconds;
	if (CurrentCacheTime != CurrentGameTime)
	{
		SetLastFrameCameraCachePOV(GetCameraCacheView());
		SetLastFrameCameraCacheTime(CurrentCacheTime);
	}

	SetCameraCachePOV(NewInfo);
	SetCameraCacheTime(CurrentGameTime);
}


void APlayerCameraManager::ProcessViewRotation(float DeltaTime, FRotator& OutViewRotation, FRotator& OutDeltaRot)
{
	SCOPE_CYCLE_COUNTER(STAT_Camera_ProcessViewRotation);
	for( int32 ModifierIdx = 0; ModifierIdx < ModifierList.Num(); ModifierIdx++ )
	{
		if( ModifierList[ModifierIdx] != NULL && 
			!ModifierList[ModifierIdx]->IsDisabled() )
		{
			if( ModifierList[ModifierIdx]->ProcessViewRotation(ViewTarget.Target, DeltaTime, OutViewRotation, OutDeltaRot) )
			{
				break;
			}
		}
	}

	// Add Delta Rotation
	OutViewRotation += OutDeltaRot;
	OutDeltaRot = FRotator::ZeroRotator;

	const bool bIsHeadTrackingAllowed =
		GEngine->XRSystem.IsValid() &&
		(GetWorld() != nullptr ? GEngine->XRSystem->IsHeadTrackingAllowedForWorld(*GetWorld()) : GEngine->XRSystem->IsHeadTrackingAllowed());
	if(bIsHeadTrackingAllowed)
	{
		// With the HMD devices, we can't limit the view pitch, because it's bound to the player's head.  A simple normalization will suffice
		OutViewRotation.Normalize();
	}
	else
	{
		// Limit Player View Axes
		LimitViewPitch( OutViewRotation, ViewPitchMin, ViewPitchMax );
		LimitViewYaw( OutViewRotation, ViewYawMin, ViewYawMax );
		LimitViewRoll( OutViewRotation, ViewRollMin, ViewRollMax );
	}
}



void APlayerCameraManager::LimitViewPitch( FRotator& ViewRotation, float InViewPitchMin, float InViewPitchMax )
{
	ViewRotation.Pitch = FMath::ClampAngle(ViewRotation.Pitch, InViewPitchMin, InViewPitchMax);
	ViewRotation.Pitch = FRotator::ClampAxis(ViewRotation.Pitch);
}

void APlayerCameraManager::LimitViewRoll( FRotator&  ViewRotation, float InViewRollMin, float InViewRollMax)
{
	ViewRotation.Roll = FMath::ClampAngle(ViewRotation.Roll, InViewRollMin, InViewRollMax);
	ViewRotation.Roll = FRotator::ClampAxis(ViewRotation.Roll);
}

void APlayerCameraManager::LimitViewYaw(FRotator& ViewRotation, float InViewYawMin, float InViewYawMax)
{
	ViewRotation.Yaw = FMath::ClampAngle(ViewRotation.Yaw, InViewYawMin, InViewYawMax);
	ViewRotation.Yaw = FRotator::ClampAxis(ViewRotation.Yaw);
}

void APlayerCameraManager::DisplayDebug(class UCanvas* Canvas, const FDebugDisplayInfo& DebugDisplay, float& YL, float& YPos)
{
	const FMinimalViewInfo& CurrentPOV = GetCameraCacheView();

	FDisplayDebugManager& DisplayDebugManager = Canvas->DisplayDebugManager;
	DisplayDebugManager.SetDrawColor(FColor(255, 255, 255));
	DisplayDebugManager.DrawString(FString::Printf(TEXT("   Camera Style:%s main ViewTarget:%s"), *CameraStyle.ToString(), *ViewTarget.Target->GetName()));
	if (PendingViewTarget.Target)
	{
		DisplayDebugManager.DrawString(FString::Printf(TEXT("   PendingViewTarget:%s"), *PendingViewTarget.Target->GetName()));
	}
	DisplayDebugManager.DrawString(FString::Printf(TEXT("   CamLoc:%s CamRot:%s FOV:%f"), *CurrentPOV.Location.ToCompactString(), *CurrentPOV.Rotation.ToCompactString(), CurrentPOV.FOV));
	DisplayDebugManager.DrawString(FString::Printf(TEXT("   AspectRatio: %1.3f"), CurrentPOV.AspectRatio));

	const float DurationPct = BlendParams.BlendTime == 0.f ? 0.f : (BlendParams.BlendTime - BlendTimeToGo) / BlendParams.BlendTime;
	const FString BlendStr = FString::Printf(TEXT("   ViewTarget Blend: From %s to %s, time remaining = %f, pct = %f"), *GetNameSafe(ViewTarget.Target), *GetNameSafe(PendingViewTarget.Target), BlendTimeToGo, DurationPct);
	DisplayDebugManager.DrawString(BlendStr);
	
	DisplayDebugManager.DrawString(TEXT("Modifiers:"));
	const FColor DisabledModifierColor(0xff949494);
	for (UCameraModifier* Modifier : ModifierList)
	{
		DisplayDebugManager.SetDrawColor(Modifier->IsDisabled() ? DisabledModifierColor : FColor::Green);
		DisplayDebugManager.DrawString(Modifier->GetName());
		Modifier->DisplayDebug(Canvas, DebugDisplay, YL, YPos);
	}
}

void APlayerCameraManager::ApplyWorldOffset(const FVector& InOffset, bool bWorldShift)
{
	Super::ApplyWorldOffset(InOffset, bWorldShift);

	FMinimalViewInfo CurrentPOV = GetCameraCacheView();
	CurrentPOV.Location += InOffset;
	SetCameraCachePOV(CurrentPOV);

	FMinimalViewInfo LastFramePOV = GetLastFrameCameraCacheView();
	LastFramePOV.Location += InOffset;
	SetLastFrameCameraCachePOV(LastFramePOV);

	ViewTarget.POV.Location+= InOffset;
	PendingViewTarget.POV.Location+= InOffset;

	CurrentPOV.Location.DiagnosticCheckNaN(TEXT("APlayerCameraManager::ApplyWorldOffset: CameraCache.POV.Location"));
	LastFramePOV.Location.DiagnosticCheckNaN(TEXT("APlayerCameraManager::ApplyWorldOffset: LastFrameCameraCache.POV.Location"));
	ViewTarget.POV.Location.DiagnosticCheckNaN(TEXT("APlayerCameraManager::ApplyWorldOffset: ViewTarget.POV.Location"));
	PendingViewTarget.POV.Location.DiagnosticCheckNaN(TEXT("APlayerCameraManager::ApplyWorldOffset: PendingViewTarget.POV.Location"));
}

TScriptInterface<class ICameraLensEffectInterface> APlayerCameraManager::FindGenericCameraLensEffect(TSubclassOf<AActor> LensEffectEmitterClass)
{
	for (int32 i = 0; i < CameraLensEffects.Num(); ++i)
	{
		const TScriptInterface<class ICameraLensEffectInterface> LensEffectInterface = CameraLensEffects[i];
		const UObject* LensEffectObject = LensEffectInterface.GetObject();

		// we have to use GetMutableDefault here because TScriptInterface cannot handle a const UObject* and requires a non-const qualified pointer.
		const TScriptInterface<const class ICameraLensEffectInterface> OtherEffectDefaultInterface = GetDefault<AActor>(LensEffectEmitterClass);

		// if the lens effect in our list is valid, and either it treats the requested effect as the same
		// or if the requested effect would treat our existing lens effect as the same...
		if (IsValid(LensEffectObject)
		&& (  (LensEffectObject->GetClass() == LensEffectEmitterClass)
		    ||(LensEffectInterface->ShouldTreatEmitterAsSame(LensEffectEmitterClass))
		    ||(OtherEffectDefaultInterface && OtherEffectDefaultInterface->ShouldTreatEmitterAsSame(LensEffectObject->GetClass()))
		   ))
		{
			// then, we can just recycle this
			return LensEffectInterface;
		}
	}

	return NULL;
}


TScriptInterface<class ICameraLensEffectInterface> APlayerCameraManager::AddGenericCameraLensEffect(TSubclassOf<AActor> LensEffectEmitterClass)
{
	if (LensEffectEmitterClass != NULL)
	{
		TScriptInterface<class ICameraLensEffectInterface> SpawnedLensEffectInterface = NULL;

		const TScriptInterface<const class ICameraLensEffectInterface> DesiredLensEffect_DefaultInterface = GetDefault<AActor>(LensEffectEmitterClass);
		const AActor* DesiredLensEffect_DefaultObject = Cast<AActor>(DesiredLensEffect_DefaultInterface.GetObject());

		if (DesiredLensEffect_DefaultInterface && !DesiredLensEffect_DefaultInterface->ShouldAllowMultipleInstances())
		{
			SpawnedLensEffectInterface = FindGenericCameraLensEffect(LensEffectEmitterClass);

			if (SpawnedLensEffectInterface != NULL)
			{
				SpawnedLensEffectInterface->NotifyRetriggered();
			}
		}

		if (SpawnedLensEffectInterface == NULL)
		{
			// spawn with viewtarget as the owner so bOnlyOwnerSee works as intended
			FActorSpawnParameters SpawnInfo;
			SpawnInfo.Owner = PCOwner->GetViewTarget();
			SpawnInfo.Instigator = GetInstigator();
			SpawnInfo.SpawnCollisionHandlingOverride = ESpawnActorCollisionHandlingMethod::AlwaysSpawn;
			SpawnInfo.ObjectFlags |= RF_Transient;	// We never want to save these into a map

			// RegisterCamera should occur before BeginPlay IMO, to do that we have to defer construction
			SpawnInfo.bDeferConstruction = true;
			
			AActor const* const EmitterCDO = LensEffectEmitterClass->GetDefaultObject<AActor>();
			FVector CamLoc;
			FRotator CamRot;
			GetCameraViewPoint(CamLoc, CamRot);
			FTransform SpawnTransform = ICameraLensEffectInterface::GetAttachedEmitterTransform(EmitterCDO, CamLoc, CamRot, GetFOVAngle());

			SpawnedLensEffectInterface = GetWorld()->SpawnActor<AActor>(LensEffectEmitterClass, SpawnTransform, SpawnInfo);

			if (SpawnedLensEffectInterface != NULL)
			{
				SpawnedLensEffectInterface->RegisterCamera(this);
				CameraLensEffects.Add(SpawnedLensEffectInterface);

				// since SpawnActor didn't fail (SpawnedLensEffectInterface was not nullptr), this check is safe.
				CastChecked<AActor>(SpawnedLensEffectInterface.GetObject(), ECastCheckedType::NullChecked)->FinishSpawning(SpawnTransform, true);
			}
		}
		
		return SpawnedLensEffectInterface;
	}

	return NULL;
}

void APlayerCameraManager::RemoveGenericCameraLensEffect(TScriptInterface<class ICameraLensEffectInterface> Emitter)
{
	CameraLensEffects.Remove(Emitter);
}

void APlayerCameraManager::ClearCameraLensEffects()
{
	for (int32 i = 0; i < CameraLensEffects.Num(); ++i)
	{
		if (AActor* ActorEffect = Cast<AActor>(CameraLensEffects[i].GetObject()))
		{
			ActorEffect->Destroy();
		}
	}

	// empty the array.  unnecessary, since destruction will call RemoveGenericCameraLensEffect,
	// but this gets it done in one fell swoop.
	CameraLensEffects.Empty();
}

/** ------------------------------------------------------------
 *  Camera Shakes
 *  ------------------------------------------------------------ */

UCameraShakeBase* APlayerCameraManager::StartCameraShake(TSubclassOf<UCameraShakeBase> ShakeClass, float Scale, ECameraShakePlaySpace PlaySpace, FRotator UserPlaySpaceRot)
{
	FAddCameraShakeParams Params(Scale, PlaySpace, UserPlaySpaceRot);
	return StartCameraShake(ShakeClass, Params);
}

UCameraShakeBase* APlayerCameraManager::StartCameraShakeFromSource(TSubclassOf<UCameraShakeBase> ShakeClass, UCameraShakeSourceComponent* SourceComponent, float Scale, ECameraShakePlaySpace PlaySpace, FRotator UserPlaySpaceRot)
{
	FAddCameraShakeParams Params(Scale, PlaySpace, UserPlaySpaceRot, SourceComponent);
	return StartCameraShake(ShakeClass, Params);
}

UCameraShakeBase* APlayerCameraManager::StartCameraShake(TSubclassOf<UCameraShakeBase> ShakeClass, const FAddCameraShakeParams& Params)
{
	if (ShakeClass && CachedCameraShakeMod)
	{
		return CachedCameraShakeMod->AddCameraShake(ShakeClass, Params);
	}

	return nullptr;
}

void APlayerCameraManager::StopCameraShake(UCameraShakeBase* ShakeInst, bool bImmediately)
{
	if (ShakeInst && CachedCameraShakeMod)
	{
		CachedCameraShakeMod->RemoveCameraShake(ShakeInst, bImmediately);
	}
}

void APlayerCameraManager::StopAllInstancesOfCameraShake(TSubclassOf<UCameraShakeBase> ShakeClass, bool bImmediately)
{
	if (ShakeClass && CachedCameraShakeMod)
	{
		CachedCameraShakeMod->RemoveAllCameraShakesOfClass(ShakeClass, bImmediately);
	}
}

void APlayerCameraManager::StopAllCameraShakes(bool bImmediately)
{
	if (CachedCameraShakeMod)
	{
		CachedCameraShakeMod->RemoveAllCameraShakes(bImmediately);
	}
}

void APlayerCameraManager::StopAllInstancesOfCameraShakeFromSource(TSubclassOf<UCameraShakeBase> ShakeClass, UCameraShakeSourceComponent* SourceComponent, bool bImmediately)
{
	if (ShakeClass && SourceComponent && CachedCameraShakeMod)
	{
		CachedCameraShakeMod->RemoveAllCameraShakesOfClassFromSource(ShakeClass, SourceComponent, bImmediately);
	}
}

void APlayerCameraManager::StopAllCameraShakesFromSource(UCameraShakeSourceComponent* SourceComponent, bool bImmediately)
{
	if (SourceComponent && CachedCameraShakeMod)
	{
		CachedCameraShakeMod->RemoveAllCameraShakesFromSource(SourceComponent, bImmediately);
	}
}

float APlayerCameraManager::CalcRadialShakeScale(APlayerCameraManager* Camera, FVector Epicenter, float InnerRadius, float OuterRadius, float Falloff)
{
	// using camera location so stuff like spectator cameras get shakes applied sensibly as well
	// need to ensure server has reasonably accurate camera position
	FVector POVLoc = Camera->GetCameraLocation();

	if (InnerRadius < OuterRadius)
	{
		float DistPct = ((Epicenter - POVLoc).Size() - InnerRadius) / (OuterRadius - InnerRadius);
		DistPct = 1.f - FMath::Clamp(DistPct, 0.f, 1.f);
		return FMath::Pow(DistPct, Falloff);
	}
	else
	{
		// ignore OuterRadius and do a cliff falloff at InnerRadius
		return ((Epicenter - POVLoc).SizeSquared() < FMath::Square(InnerRadius)) ? 1.f : 0.f;
	}
}


void APlayerCameraManager::PlayWorldCameraShake(UWorld* InWorld, TSubclassOf<class UCameraShakeBase> Shake, FVector Epicenter, float InnerRadius, float OuterRadius, float Falloff, bool bOrientShakeTowardsEpicenter )
{
	for( FConstPlayerControllerIterator Iterator = InWorld->GetPlayerControllerIterator(); Iterator; ++Iterator )
	{
		APlayerController* PlayerController = Iterator->Get();
		if (PlayerController && PlayerController->PlayerCameraManager != NULL)
		{
			float ShakeScale = CalcRadialShakeScale(PlayerController->PlayerCameraManager, Epicenter, InnerRadius, OuterRadius, Falloff);

			if (bOrientShakeTowardsEpicenter && PlayerController->GetPawn() != NULL)
			{
				const FVector CamLoc = PlayerController->PlayerCameraManager->GetCameraLocation();
				PlayerController->ClientStartCameraShake(Shake, ShakeScale, ECameraShakePlaySpace::UserDefined, (Epicenter - CamLoc).Rotation());
			}
			else
			{
				PlayerController->ClientStartCameraShake(Shake, ShakeScale);
			}
		}
	}
}


/** ------------------------------------------------------------
 *  Camera fades
 *  ------------------------------------------------------------ */

void APlayerCameraManager::StartCameraFade(float FromAlpha, float ToAlpha, float InFadeTime, FLinearColor InFadeColor, bool bInFadeAudio, bool bInHoldWhenFinished)
{
	bEnableFading = true;

	FadeColor = InFadeColor;
	FadeAlpha = FVector2D(FromAlpha, ToAlpha);
	FadeTime = InFadeTime;
	FadeTimeRemaining = InFadeTime;
	bFadeAudio = bInFadeAudio;

	if (bInFadeAudio && OnAudioFadeChangeEvent.IsBound())
	{
		const bool bFadeOut = FromAlpha < ToAlpha || FMath::IsNearlyEqual(ToAlpha, 1.0f);
		OnAudioFadeChangeEvent.Broadcast(bFadeOut, FadeTime);
	}

	bAutoAnimateFade = true;
	bHoldFadeWhenFinished = bInHoldWhenFinished;
}

void APlayerCameraManager::StopCameraFade()
{
	if (bEnableFading == true)
	{
		// Make sure FadeAmount finishes at the desired value
		FadeAmount = FadeAlpha.Y;
		bEnableFading = false;
		StopAudioFade();
	}
}

void APlayerCameraManager::SetManualCameraFade(float InFadeAmount, FLinearColor Color, bool bInFadeAudio)
{
	bEnableFading = true;
	FadeColor = Color;
	FadeAmount = InFadeAmount;
	bFadeAudio = bInFadeAudio;

	bAutoAnimateFade = false;
	StopAudioFade();
	FadeTimeRemaining = 0.0f;
}

void APlayerCameraManager::ForEachCameraModifier(TFunctionRef<bool(UCameraModifier*)> Fn)
{
	// Local copy the modifiers array in case it get when calling the lambda on each modifiers
	TArray<TObjectPtr<UCameraModifier>> LocalModifierList = ModifierList;

	// Loop through each camera modifier
	for (int32 ModifierIdx = 0; ModifierIdx < LocalModifierList.Num(); ++ModifierIdx)
	{
		if (!Fn(LocalModifierList[ModifierIdx]))
		{
			return;
		}
	}
}

void APlayerCameraManager::SetCameraCachePOV(const FMinimalViewInfo& InPOV)
{
	CameraCachePrivate.POV = InPOV;
}

void APlayerCameraManager::SetLastFrameCameraCachePOV(const FMinimalViewInfo& InPOV)
{
	LastFrameCameraCachePrivate.POV = InPOV;
}

const FMinimalViewInfo& APlayerCameraManager::GetCameraCacheView() const
{
	return CameraCachePrivate.POV;
}

const FMinimalViewInfo& APlayerCameraManager::GetLastFrameCameraCacheView() const
{
	return LastFrameCameraCachePrivate.POV;
}

void APlayerCameraManager::GetCameraCacheLocationAndRotationForServer(FVector& OutLocation, FRotator& OutRotation) const
{
	const FMinimalViewInfo& ViewInfo = GetCameraCacheView();
	OutLocation = ViewInfo.Location;
	OutRotation = ViewInfo.Rotation;
}

void APlayerCameraManager::GetLastFrameCameraCacheLocationAndRotationForServer(FVector& OutLocation, FRotator& OutRotation) const
{
	const FMinimalViewInfo& ViewInfo = GetLastFrameCameraCacheView();
	OutLocation = ViewInfo.Location;
	OutRotation = ViewInfo.Rotation;
}

//////////////////////////////////////////////////////////////////////////
// FTViewTarget

void FTViewTarget::SetNewTarget(AActor* NewTarget)
{
	Target = NewTarget;
}

APawn* FTViewTarget::GetTargetPawn() const
{
	if (APawn* Pawn = Cast<APawn>(Target))
	{
		return Pawn;
	}
	else if (AController* Controller = Cast<AController>(Target))
	{
		return Controller->GetPawn();
	}
	else
	{
		return NULL;
	}
}

bool FTViewTarget::Equal(const FTViewTarget& OtherTarget) const
{
	//@TODO: Should I compare Controller too?
	return (Target == OtherTarget.Target) && (PlayerState == OtherTarget.PlayerState) && POV.Equals(OtherTarget.POV);
}


void FTViewTarget::CheckViewTarget(APlayerController* OwningController)
{
	check(OwningController);

	if (Target == NULL)
	{
		Target = OwningController;
	}

	// Update ViewTarget PlayerState (used to follow same player through pawn transitions, etc., when spectating)
	if (Target == OwningController) 
	{	
		PlayerState = NULL;
	}
	else if (AController* TargetAsController = Cast<AController>(Target))
	{
		PlayerState = TargetAsController->PlayerState;
	}
	else if (APawn* TargetAsPawn = Cast<APawn>(Target))
	{
		PlayerState = TargetAsPawn->GetPlayerState();
	}
	else if (APlayerState* TargetAsPlayerState = Cast<APlayerState>(Target))
	{
		PlayerState = TargetAsPlayerState;
	}
	else
	{
		PlayerState = NULL;
	}

	if (PlayerState && IsValidChecked(PlayerState))
	{
		if (!IsValid(Target) || !Cast<APawn>(Target) || (CastChecked<APawn>(Target)->GetPlayerState() != PlayerState) )
		{
			Target = NULL;

			// not viewing pawn associated with VT.PlayerState, so look for one
			// Assuming on server, so PlayerState Owner is valid
			if (PlayerState->GetOwner() == NULL)
			{
				PlayerState = NULL;
			}
			else
			{
				if (AController* PlayerStateOwner = Cast<AController>(PlayerState->GetOwner()))
				{
					AActor* PlayerStateViewTarget = PlayerStateOwner->GetPawn();
					if (IsValid(OwningController) && IsValid(PlayerStateViewTarget))
					{
						OwningController->PlayerCameraManager->AssignViewTarget(PlayerStateViewTarget, *this);
					}
					else
					{
						Target = PlayerState; // this will cause it to update to the next Pawn possessed by the player being viewed
					}
				}
				else
				{
					PlayerState = NULL;
				}
			}
		}
	}

	if (IsValid(OwningController) && !IsValid(Target))
	{
		if (OwningController->GetPawn() && !OwningController->GetPawn()->IsPendingKillPending() )
		{
			OwningController->PlayerCameraManager->AssignViewTarget(OwningController->GetPawn(), *this);
		}
		else
		{
			OwningController->PlayerCameraManager->AssignViewTarget(OwningController, *this);
		}
	}
}
