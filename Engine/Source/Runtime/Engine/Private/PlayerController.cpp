// Copyright Epic Games, Inc. All Rights Reserved.

#include "GameFramework/PlayerController.h"
#include "Engine/GameInstance.h"
#include "Engine/GameViewportClient.h"
#include "Materials/MaterialInterface.h"
#include "GameFramework/CheatManagerDefines.h"
#include "Misc/PackageName.h"
#include "GameFramework/ForceFeedbackEffect.h"
#include "TimerManager.h"
#include "Physics/Experimental/PhysScene_Chaos.h"
#include "SceneView.h"
#include "Camera/CameraActor.h"
#include "UObject/Package.h"
#include "EngineStats.h"
#include "Engine/Canvas.h"
#include "GameFramework/PlayerStart.h"
#include "Components/SkeletalMeshComponent.h"
#include "Components/AudioComponent.h"
#include "Components/ForceFeedbackComponent.h"
#include "Kismet/GameplayStatics.h"
#include "Engine/Engine.h"
#include "Engine/LevelStreaming.h"
#include "Engine/LocalPlayer.h"
#include "Engine/NetConnection.h"
#include "ContentStreaming.h"
#include "DrawDebugHelpers.h"
#include "EngineUtils.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/Application/SlateUser.h"
#include "Widgets/SViewport.h"
#include "Engine/Console.h"
#include "Net/UnrealNetwork.h"
#include "Net/Core/Misc/NetConditionGroupManager.h"
#include "Net/Core/PushModel/PushModel.h"
#include "Engine/WorldComposition.h"
#include "Engine/LevelScriptActor.h"
#include "GameFramework/GameNetworkManager.h"
#include "Interfaces/NetworkPredictionInterface.h"
#include "Net/OnlineEngineInterface.h"
#include "GameFramework/OnlineSession.h"
#include "IHeadMountedDisplay.h"
#include "IXRTrackingSystem.h"
#include "IXRCamera.h"
#include "GameFramework/TouchInterface.h"
#include "DisplayDebugHelpers.h"
#include "MoviePlayerProxy.h"
#include "GameFramework/PawnMovementComponent.h"
#include "GameFramework/SpectatorPawn.h"
#include "GameFramework/HUD.h"
#include "Engine/InputDelegateBinding.h"
#include "Widgets/Input/SVirtualJoystick.h"
#include "GameFramework/LocalMessage.h"
#include "GameFramework/CheatManager.h"
#include "GameFramework/InputSettings.h"
#include "GameFramework/PlayerState.h"
#include "GameFramework/GameStateBase.h"
#include "Haptics/HapticFeedbackEffect_Base.h"
#include "Engine/ChildConnection.h"
#include "VisualLogger/VisualLogger.h"
#include "Slate/SceneViewport.h"
#include "Engine/NetworkObjectList.h"
#include "GameFramework/GameSession.h"
#include "GameMapsSettings.h"
#include "Particles/EmitterCameraLensEffectBase.h"
#include "LevelUtils.h"
#include "WorldPartition/WorldPartitionSubsystem.h"
#include "PBDRigidsSolver.h"
#include "PhysicsEngine/PhysicsSettings.h"
#include "DownlinkBandwidthManager.h"

#include "Iris/ReplicationSystem/ReplicationSystem.h"
#include "Net/Iris/ReplicationSystem/EngineReplicationBridge.h"
#include "Net/Iris/ReplicationSystem/ReplicationSystemUtil.h"

#if UE_WITH_REMOTE_OBJECT_HANDLE
#include "AutoRTFM.h"
#include "UObject/UObjectMigrationContext.h"
#include "UObject/RemoteExecutor.h"
#include "UObject/RemoteObject.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(PlayerController)

DEFINE_LOG_CATEGORY(LogPlayerController);

#define LOCTEXT_NAMESPACE "PlayerController"

DECLARE_CYCLE_STAT(TEXT("PC Tick Actor"), STAT_PC_TickActor, STATGROUP_PlayerController);
DECLARE_CYCLE_STAT(TEXT("PC Tick Input"), STAT_PC_TickInput, STATGROUP_PlayerController);
DECLARE_CYCLE_STAT(TEXT("PC Build Input Stack"), STAT_PC_BuildInputStack, STATGROUP_PlayerController);
DECLARE_CYCLE_STAT(TEXT("PC Process Input Stack"), STAT_PC_ProcessInputStack, STATGROUP_PlayerController);

// CVars
namespace UE::Gameplay::CVars
{
	// Resync timestamps on pawn ack
	int32 NetResetServerPredictionDataOnPawnAck = 1;
	static FAutoConsoleVariableRef CVarNetResetServerPredictionDataOnPawnAck(
		TEXT("PlayerController.NetResetServerPredictionDataOnPawnAck"),
		NetResetServerPredictionDataOnPawnAck,
		TEXT("Whether to reset server prediction data for the possessed Pawn when the pawn ack handshake completes.\n")
		TEXT("0: Disable, 1: Enable"),
		ECVF_Default);

	int32 ForceUsingCameraAsStreamingSource = 0;
	static FAutoConsoleVariableRef CVarForceUsingCameraAsStreamingSource(
		TEXT("wp.Runtime.PlayerController.ForceUsingCameraAsStreamingSource"),
		ForceUsingCameraAsStreamingSource,
		TEXT("Whether to force the use of the camera as the streaming source for World Partition. By default the player pawn is used.\n")
		TEXT("0: Use pawn as streaming source, 1: Use camera as streaming source"));

	bool bIsPlayerControllerPushBased = false;
	static FAutoConsoleVariableRef CVarIsPlayerControllerPushBased(
		TEXT("PlayerController.IsPushBased"), bIsPlayerControllerPushBased,
		TEXT("If true, APlayerController's replicated properties will use push-based networking, and will therefore need to be marked dirty when changed."),
		ECVF_Default);

	bool bSortInputStackByPriority = true;
	static FAutoConsoleVariableRef CVarSortInputStackByPriority(
		TEXT("PlayerController.Input.SortInputStackByPriority"), 
		bSortInputStackByPriority,
		TEXT("If true, APlayerController's input component stack will be sorted by priority after being built. False will maintain 'legacy' ordering."),
		ECVF_Default);

	extern bool bAlwaysNotifyClientOnControllerChange;
}

namespace NetworkPhysicsCvars
{
#if (UE_BUILD_SHIPPING || UE_BUILD_TEST)
		int32 EnableDebugRPC = 0;
#else
		int32 EnableDebugRPC = 1;
#endif

	int32 TickOffsetUpdateIntervalTime = 100;
	FAutoConsoleVariableRef CVarTickOffsetUpdateIntervalTime(TEXT("np2.TickOffsetUpdateIntervalTime"), TickOffsetUpdateIntervalTime, TEXT("Value in milliseconds, default 100. How long time between syncing the tick offset between client and server. Deactivate syncing by setting value 0."));

	int32 TickOffsetBufferTime = 90;
	FAutoConsoleVariableRef CVarTickOffsetBufferTime(TEXT("np2.TickOffsetBufferTime"), TickOffsetBufferTime, TEXT("Value in milliseconds, default 60. Additional offset to be added to the local to server offset used by network prediction, this results in a buffer server-side for incoming data that uses the client/server physics offset."));

	int32 TickOffsetCorrectionSizeTimeLimit = 2000;
	FAutoConsoleVariableRef CVarTickOffsetCorrectionSizeTimeLimit(TEXT("np2.TickOffsetCorrectionSizeTimeLimit"), TickOffsetCorrectionSizeTimeLimit, TEXT("Value in milliseconds. Note: Keep this equal to or larger than np2.TickOffsetBufferTime. If the client gets out of sync with physics ticks and the desync is larger than this value, reset the offset."));

	int32 TickOffsetCorrectionTimeLimit = 5000;
	FAutoConsoleVariableRef CVarTickOffsetCorrectionTimeLimit(TEXT("np2.TickOffsetCorrectionTimeLimit"), TickOffsetCorrectionTimeLimit, TEXT("Value in milliseconds. If the client gets out of sync with physics ticks and can't get in sync again for this amount of time, reset the offset."));

	float TimeDilationAmount = 0.025f;
	FAutoConsoleVariableRef CVarTimeDilationAmount(TEXT("np2.TimeDilationAmount"), TimeDilationAmount, TEXT("Server-side CVar, Disable TimeDilation by setting to 0 | Value is in percent where 0.01 = 1% dilation. Example: 1.0/0.01 = 100, meaning that over the time it usually takes to tick 100 physics steps we will tick 99 or 101 depending on if we dilate up or down."));

	bool TimeDilationEscalation = true;
	FAutoConsoleVariableRef CVarTimeDilationEscalation(TEXT("np2.TimeDilationEscalation"), TimeDilationEscalation, TEXT("Server-side CVar, Dilate the time more depending on how many ticks we need to adjust. When set to false we use the set TimeDilationAmount and wait the amount of time it takes to perform correct the offset. When set to true we multiply the TimeDilationAmount with the buffer offset count which will correct the offset in one TimeDilationAmount cycle."));

	float TimeDilationEscalationDecay = 0.05f;
	FAutoConsoleVariableRef CVarTimeDilationEscalationDecay(TEXT("np2.TimeDilationEscalationDecay"), TimeDilationEscalationDecay, TEXT("Value is a multiplier, Default: 0.05. For each escalated TimeDilation amount, also decay by this much. Disable by setting to 0."));

	float TimeDilationEscalationDecayMax = 0.5f;
	FAutoConsoleVariableRef CVarTimeDilationEscalationDecayMax(TEXT("np2.TimeDilationEscalationDecayMax"), TimeDilationEscalationDecayMax, TEXT("Value is a multiplier, Default: 0.5. The max decay value for escalated time dilation. Lower value means higher decay."));

	float TimeDilationMax = 1.25f;
	FAutoConsoleVariableRef CVarTimeDilationMax(TEXT("np2.TimeDilationMax"), TimeDilationMax, TEXT("Max value of the time dilation multiplier."));

	float TimeDilationMin = 0.75f;
	FAutoConsoleVariableRef CVarTimeDilationMin(TEXT("np2.TimeDilationMin"), TimeDilationMin, TEXT("Min value of the time dilation multiplier"));
}

const float RetryClientRestartThrottleTime = 0.5f;
const float RetryServerAcknowledgeThrottleTime = 0.25f;
const float RetryServerCheckSpectatorThrottleTime = 0.25f;

// Note: This value should be sufficiently small such that it is considered to be in the past before RetryClientRestartThrottleTime and RetryServerAcknowledgeThrottleTime.
const float ForceRetryClientRestartTime = -100.0f;

//////////////////////////////////////////////////////////////////////////

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
/** Used to display the force feedback history of what was played most recently. */
struct FForceFeedbackEffectHistoryEntry
{
	FActiveForceFeedbackEffect LastActiveForceFeedbackEffect;
	float TimeShown;

	FForceFeedbackEffectHistoryEntry(FActiveForceFeedbackEffect LastActiveFFE, float Time)
	{
		LastActiveForceFeedbackEffect = LastActiveFFE;
		TimeShown = Time;
	}
};
#endif

//////////////////////////////////////////////////////////////////////////
// APlayerController

APlayerController::APlayerController(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	NetPriority = 3.0f;
	CheatClass = UCheatManager::StaticClass();
	ClientCap = 0;
	LocalPlayerCachedLODDistanceFactor = 1.0f;
	bIsUsingStreamingVolumes = true;
	PrimaryActorTick.TickGroup = TG_PrePhysics;
	PrimaryActorTick.bTickEvenWhenPaused = true;
	bAllowTickBeforeBeginPlay = true;
	bShouldPerformFullTickWhenPaused = false;
	LastRetryPlayerTime = ForceRetryClientRestartTime;
	DefaultMouseCursor = EMouseCursor::Default;
	DefaultClickTraceChannel = ECollisionChannel::ECC_Visibility;
	HitResultTraceDistance = 100000.f;

	LastMovementUpdateTime = 0.f;
	LastMovementHitch = 0.f;

	bCinemaDisableInputMove = false;
	bCinemaDisableInputLook = false;

	bInputEnabled = true;
	bEnableTouchEvents = true;
	bForceFeedbackEnabled = true;
	ForceFeedbackScale = 1.f;

	// default to true; won't do anything if enable motion controls in input settings isn't also true
	SetMotionControlsEnabled(true);

	bEnableStreamingSource = true;
	bStreamingSourceShouldActivate = true;
	bStreamingSourceShouldBlockOnSlowStreaming = true;
	StreamingSourcePriority = EStreamingSourcePriority::Default;

	bAutoManageActiveCameraTarget = true;
	bRenderPrimitiveComponents = true;
	SmoothTargetViewRotationSpeed = 20.f;
	bHidePawnInCinematicMode = false;

	bIsPlayerController = true;
	bIsLocalPlayerController = false;
	bDisableHaptics = false;
	bShouldFlushInputWhenViewportFocusChanges = true;

	ClickEventKeys.Add(EKeys::LeftMouseButton);

	TickOffsetBufferTime = NetworkPhysicsCvars::TickOffsetBufferTime;

	if (RootComponent)
	{
		// We want to drive rotation with ControlRotation regardless of attachment state.
		RootComponent->SetUsingAbsoluteRotation(true);
	}

#if UE_ENABLE_DEBUG_DRAWING
	CurrentInputModeDebugString = TEXT("Default");
#endif	// UE_ENABLE_DEBUG_DRAWING
}

float APlayerController::GetNetPriority(const FVector& ViewPos, const FVector& ViewDir, AActor* Viewer, AActor* ViewTarget, UActorChannel* InChannel, float Time, bool bLowBandwidth)
{
	if ( Viewer == this )
	{
		Time *= 4.f;
	}
	return NetPriority * Time;
}

const AActor* APlayerController::GetNetOwner() const
{
	return this;
}

UPlayer* APlayerController::GetNetOwningPlayer() 
{
	return Player;
}

class UPlayer* APlayerController::GetNetOwningPlayerAnyRole()
{
	return Player;
}

bool APlayerController::HasNetOwner() const
{
	// Player controllers are their own net owners
	return true;
}

UNetConnection* APlayerController::GetNetConnection() const
{
	// A controller without a player has no "owner"
	return (Player != NULL) ? NetConnection : NULL;
}

bool APlayerController::DestroyNetworkActorHandled()
{
	UNetConnection* Connection = Cast<UNetConnection>(Player);
	if (Connection)
	{
		Connection->GracefulClose(ENetCloseResult::ControlChannelClose);
		return true;
	}

	return false;
}

bool APlayerController::IsLocalController() const
{
	// Never local on dedicated server. IsServerOnly() is checked at compile time and optimized out appropriately.
	if (FPlatformProperties::IsServerOnly())
	{
		checkSlow(!bIsLocalPlayerController);
		return false;
	}

	// Fast path if we have this bool set.
	if (bIsLocalPlayerController)
	{
		return true;
	}


	ENetMode NetMode = GetNetMode();
	if (NetMode == NM_DedicatedServer)
	{
		// This is still checked for the PIE case, which would not be caught in the IsServerOnly() check above.
		checkSlow(!bIsLocalPlayerController);
		return false;
	}

		
	// If the NetDriver is already gone (e.g., seamless travel back to standalone/PIE teardown),
	// classify "local" by presence of a ULocalPlayer on this machine. Remote clients' PCs on a listen
	// server do NOT have a ULocalPlayer, but true clients and the listen host do.
	if (GetNetDriver() == nullptr)
	{
		const bool bHasLocalPlayer = Cast<ULocalPlayer>(Player) != nullptr;
		if (bHasLocalPlayer)
		{
			bIsLocalPlayerController = true;
			return true;
		}
		return false;
	}


	if (NetMode == NM_Client || NetMode == NM_Standalone)
	{
		// Clients or Standalone only receive their own PC. We are not ROLE_AutonomousProxy until after PostInitializeComponents so we can't check that.
		bIsLocalPlayerController = true;
		return true;
	}

	return bIsLocalPlayerController;
}

void APlayerController::FailedToSpawnPawn()
{
	Super::FailedToSpawnPawn();
	ChangeState(NAME_Inactive);
	ClientGotoState(NAME_Inactive);
}

FName APlayerController::NetworkRemapPath(FName InPackageName, bool bReading)
{
	// For PIE Networking: remap the packagename to our local PIE packagename
	FString PackageNameStr = InPackageName.ToString();
	GEngine->NetworkRemapPath(GetNetConnection(), PackageNameStr, bReading);
	return FName(*PackageNameStr);
}

/// @cond DOXYGEN_WARNINGS

void APlayerController::ClientUpdateLevelStreamingStatus_Implementation(FName PackageName, bool bNewShouldBeLoaded, bool bNewShouldBeVisible, bool bNewShouldBlockOnLoad, int32 LODIndex, FNetLevelVisibilityTransactionId TransactionId, bool bNewShouldBlockOnUnload)
{
	PackageName = NetworkRemapPath(PackageName, true);
	
	UWorld* World = GetWorld();

	// Distance dependent streaming levels should be controlled by client only
	if (World && World->WorldComposition)
	{
		if (World->WorldComposition->IsDistanceDependentLevel(PackageName))
		{
			return;
		}
	}

	// Search for the streaming level object by name
	ULevelStreaming* LevelStreamingObject = FLevelUtils::FindStreamingLevel(World, PackageName);

	// Skip if streaming level object doesn't allow replicating the status
	if (LevelStreamingObject && !LevelStreamingObject->CanReplicateStreamingStatus())
	{
		return;
	}

	// if we're about to commit a map change, we assume that the streaming update is based on the to be loaded map and so defer it until that is complete
	if (GEngine->ShouldCommitPendingMapChange(World))
	{
		GEngine->AddNewPendingStreamingLevel(World, PackageName, bNewShouldBeLoaded, bNewShouldBeVisible, LODIndex);		
	}
	else if (LevelStreamingObject)
	{
		// If we're unloading any levels, we need to request a one frame delay of garbage collection to make sure it happens after the level is actually unloaded
		if (LevelStreamingObject->ShouldBeLoaded() && !bNewShouldBeLoaded)
		{
			GEngine->DelayGarbageCollection();
		}

		LevelStreamingObject->SetShouldBeLoaded(bNewShouldBeLoaded);
		LevelStreamingObject->SetShouldBeVisible(bNewShouldBeVisible);
		LevelStreamingObject->bShouldBlockOnLoad = bNewShouldBlockOnLoad;
		LevelStreamingObject->bShouldBlockOnUnload = bNewShouldBlockOnUnload;
		LevelStreamingObject->SetLevelLODIndex(LODIndex);
		LevelStreamingObject->UpdateNetVisibilityTransactionState(bNewShouldBeVisible, TransactionId);
	}
	else
	{
		UE_LOGF(LogStreaming, Log, "Unable to find streaming object %ls", *PackageName.ToString() );
	}
}

void APlayerController::ClientUpdateMultipleLevelsStreamingStatus_Implementation( const TArray<FUpdateLevelStreamingLevelStatus>& LevelStatuses )
{
	for( const FUpdateLevelStreamingLevelStatus& LevelStatus : LevelStatuses )
	{
		ClientUpdateLevelStreamingStatus_Implementation(LevelStatus.PackageName, LevelStatus.bNewShouldBeLoaded, LevelStatus.bNewShouldBeVisible, LevelStatus.bNewShouldBlockOnLoad, LevelStatus.LODIndex, FNetLevelVisibilityTransactionId(), LevelStatus.bNewShouldBlockOnUnload);
	}
}

void APlayerController::ClientAckUpdateLevelVisibility_Implementation(FName PackageName, FNetLevelVisibilityTransactionId TransactionId, bool bClientAckCanMakeVisibleResponse)
{
	if (ensureAlwaysMsgf(TransactionId.IsClientTransaction(), TEXT("APlayerController::ClientAckUpdateLevelVisibility Expected TransactionId to be ClientTransaction")))
	{
		// find streaming levels and update request id
		PackageName = NetworkRemapPath(PackageName, true);

		if (ULevelStreaming* LevelStreamingObject = FLevelUtils::FindStreamingLevel(GetWorld(), PackageName))
		{
			FAckNetVisibilityTransaction::Call(LevelStreamingObject, TransactionId, bClientAckCanMakeVisibleResponse);
		}
	}
}

void APlayerController::ClientFlushLevelStreaming_Implementation()
{
	UWorld* World = GetWorld();
	// request level streaming be flushed next frame
	World->UpdateLevelStreaming();
	World->bRequestedBlockOnAsyncLoading = true;
	// request GC as soon as possible to remove any unloaded levels from memory
	GEngine->ForceGarbageCollection();
}


void APlayerController::ServerUpdateLevelVisibility_Implementation(const FUpdateLevelVisibilityLevelInfo& LevelVisibility)
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_ServerUpdateLevelVisibility);

	UNetConnection* Connection = Cast<UNetConnection>(Player);
	if (Connection != NULL)
	{
		FUpdateLevelVisibilityLevelInfo LevelVisibilityCopy = LevelVisibility;
		LevelVisibilityCopy.PackageName = NetworkRemapPath(LevelVisibilityCopy.PackageName, true);
		// FileName and packageName might differ so we have to remap the filename as well.
		LevelVisibilityCopy.FileName = NetworkRemapPath(LevelVisibilityCopy.FileName, true);

		Connection->UpdateLevelVisibility(LevelVisibilityCopy);

		// If this is a client instigated request respond with the request id so that the client knows that we have received the visibility update
		if (LevelVisibilityCopy.VisibilityRequestId.IsClientTransaction())
		{
			// In case a making visible request was done but querying the server is not supported, reponse true to unblock the client
			const bool bClientAckCanMakeVisibleResponse = !FLevelUtils::SupportsMakingVisibleTransactionRequests(GetWorld()) || (LevelVisibilityCopy.bTryMakeVisible && Connection->GetClientMakingVisibleLevelNames().Contains(LevelVisibilityCopy.PackageName));
			ClientAckUpdateLevelVisibility(LevelVisibility.PackageName, LevelVisibilityCopy.VisibilityRequestId, bClientAckCanMakeVisibleResponse);
		}
	}
}

bool APlayerController::ServerUpdateLevelVisibility_Validate(const FUpdateLevelVisibilityLevelInfo& LevelVisibility)
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_ServerUpdateLevelVisibility_Validate);

	RPC_VALIDATE(LevelVisibility.PackageName.IsValid());

	FText Reason;

	if (!FPackageName::IsValidLongPackageName(LevelVisibility.PackageName.ToString(), true, &Reason))
	{
		UE_LOGF(LogPlayerController, Warning, "ServerUpdateLevelVisibility() Invalid package name: %ls (%ls)", *LevelVisibility.PackageName.ToString(), *Reason.ToString());
		return false;
	}

	return true;
}

void APlayerController::ServerUpdateContainerVisibility_Implementation(UObject* Container, bool bVisible)
{
	if (UNetConnection* Connection = Cast<UNetConnection>(Player))
	{
		Connection->UpdateContainerVisibility(Container, bVisible);
	}
}

void APlayerController::ServerUpdateMultipleLevelsVisibility_Implementation( const TArray<FUpdateLevelVisibilityLevelInfo>& LevelVisibilities )
{
	for(const FUpdateLevelVisibilityLevelInfo& LevelVisibility : LevelVisibilities)
	{
		ServerUpdateLevelVisibility_Implementation(LevelVisibility);
	}
}

bool APlayerController::ServerUpdateMultipleLevelsVisibility_Validate( const TArray<FUpdateLevelVisibilityLevelInfo>& LevelVisibilities )
{
	for(const FUpdateLevelVisibilityLevelInfo& LevelVisibility : LevelVisibilities)
	{
		if(!ServerUpdateLevelVisibility_Validate(LevelVisibility))
		{
			return false;
		}
	}

	return true;
}

void APlayerController::ClientAddTextureStreamingLoc_Implementation(FVector InLoc, float Duration, bool bOverrideLocation )
{
	if (!IStreamingManager::HasShutdown())
	{
		IStreamingManager::Get().AddViewLocation(InLoc, 1.0f, bOverrideLocation, Duration);
	}
}

/// @endcond

void APlayerController::SetNetSpeed(int32 NewSpeed)
{
	UNetDriver* Driver = GetWorld()->GetNetDriver();
	if (Player != NULL && Driver != NULL)
	{
		Player->CurrentNetSpeed = FMath::Clamp(NewSpeed, 1800, Driver->MaxClientRate);
		if (Driver->ServerConnection != NULL)
		{
			Driver->ServerConnection->CurrentNetSpeed = Player->CurrentNetSpeed;
		}
	}
}

FString APlayerController::ConsoleCommand(const FString& Cmd, bool bWriteToLog)
{
	if (Player != nullptr)
	{
		FString Result;
#if UE_WITH_REMOTE_OBJECT_HANDLE && UE_AUTORTFM
		if (!AutoRTFM::IsTransactional())
		{
			static FName TransactionalWorkName("APlayerController::ConsoleComand");
			UE::RemoteExecutor::ExecuteTransactional(TransactionalWorkName, [this, &Result, Cmd, bWriteToLog]()
			{
				Result = Player->ConsoleCommand(Cmd, bWriteToLog); 
			});
		}
		else
#endif
		{
			Result = Player->ConsoleCommand(Cmd, bWriteToLog);
		}
		return Result;		
	}

	return TEXT("");
}

void APlayerController::CleanUpAudioComponents()
{
	TInlineComponentArray<UAudioComponent*> Components;
	GetComponents(Components);

	for(int32 CompIndex = 0; CompIndex < Components.Num(); CompIndex++)
	{
		UAudioComponent* AComp = Components[CompIndex];
		if (AComp->Sound == NULL)
		{
			AComp->DestroyComponent();
		}
	}
}

AActor* APlayerController::GetViewTarget() const
{
	AActor* CameraManagerViewTarget = PlayerCameraManager ? PlayerCameraManager->GetViewTarget() : NULL;

	return CameraManagerViewTarget ? CameraManagerViewTarget : const_cast<APlayerController*>(this);
}

void APlayerController::SetViewTarget(class AActor* NewViewTarget, struct FViewTargetTransitionParams TransitionParams)
{
	if (PlayerCameraManager)
	{
		PlayerCameraManager->SetViewTarget(NewViewTarget, TransitionParams);
	}
}


void APlayerController::AutoManageActiveCameraTarget(AActor* SuggestedTarget)
{
	if (bAutoManageActiveCameraTarget)
	{
		// See if there is a CameraActor with an auto-activate index that matches us.
		if (GetNetMode() == NM_Client)
		{
			// Clients don't know their own index on the server, so they have to trust that if they use a camera with an auto-activate index, that's their own index.
			ACameraActor* CurrentCameraActor = Cast<ACameraActor>(GetViewTarget());
			if (CurrentCameraActor)
			{
				const int32 CameraAutoIndex = CurrentCameraActor->GetAutoActivatePlayerIndex();
				if (CameraAutoIndex != INDEX_NONE)
				{					
					return;
				}
			}
		}
		else
		{
			// See if there is a CameraActor in the level that auto-activates for this PC.
			ACameraActor* AutoCameraTarget = GetAutoActivateCameraForPlayer();
			if (AutoCameraTarget)
			{
				SetViewTarget(AutoCameraTarget);
				return;
			}
		}

		// No auto-activate CameraActor, so use the suggested target.
		SetViewTarget(SuggestedTarget);
	}
}



ACameraActor* APlayerController::GetAutoActivateCameraForPlayer() const
{
	if (GetNetMode() == NM_Client)
	{
		// Clients get their view target replicated, they don't use placed cameras because they don't know their own index.
		return NULL;
	}

	UWorld* CurWorld = GetWorld();
	if (!CurWorld)
	{
		return NULL;
	}

	// Only bother if there are any registered cameras.
	FConstCameraActorIterator CameraIterator = CurWorld->GetAutoActivateCameraIterator();
	if (!CameraIterator)
	{
		return NULL;
	}

	// Find our player index
	int32 IterIndex = 0;
	int32 PlayerIndex = INDEX_NONE;
	for( FConstPlayerControllerIterator Iterator = CurWorld->GetPlayerControllerIterator(); Iterator; ++Iterator, ++IterIndex )
	{
		const APlayerController* PlayerController = Iterator->Get();
		if (PlayerController == this)
		{
			PlayerIndex = IterIndex;
			break;
		}
	}

	if (PlayerIndex != INDEX_NONE)
	{
		// Find the matching camera
		for( /*CameraIterater initialized above*/; CameraIterator; ++CameraIterator)
		{
			ACameraActor* CameraActor = CameraIterator->Get();
			if (CameraActor && CameraActor->GetAutoActivatePlayerIndex() == PlayerIndex)
			{
				return CameraActor;
			}
		}
	}

	return NULL;
}

/// @cond DOXYGEN_WARNINGS

bool APlayerController::ServerNotifyLoadedWorld_Validate(FName WorldPackageName)
{
	RPC_VALIDATE( WorldPackageName.IsValid() );
	return true;
}

void APlayerController::ServerNotifyLoadedWorld_Implementation(FName WorldPackageName)
{
	WorldPackageName = NetworkRemapPath(WorldPackageName, true);

	UE_LOGF(LogPlayerController, Verbose, "APlayerController::ServerNotifyLoadedWorld_Implementation: Client loaded %ls", *WorldPackageName.ToString());

	UWorld* CurWorld = GetWorld();

	// Only valid for calling, for PC's in the process of seamless traveling
	// NOTE: SeamlessTravelCount tracks client seamless travel, through the serverside gameplay code; this should not be replaced.
	if (CurWorld != NULL && !CurWorld->IsNetMode(NM_Client) && SeamlessTravelCount > 0 && LastCompletedSeamlessTravelCount < SeamlessTravelCount)
	{
		// Update our info on what world the client is in
		UNetConnection* const Connection = Cast<UNetConnection>(Player);

		if (Connection != NULL)
		{
			Connection->SetClientWorldPackageName(WorldPackageName);
		}

		// if both the server and this client have completed the transition, handle it
		FSeamlessTravelHandler& SeamlessTravelHandler = GEngine->SeamlessTravelHandlerForWorld(CurWorld);
		AGameModeBase* CurGameMode = CurWorld->GetAuthGameMode();

		if (!SeamlessTravelHandler.IsInTransition() && WorldPackageName == CurWorld->GetOutermost()->GetFName() && CurGameMode != NULL)
		{
			AController* TravelPlayer = this;
			CurGameMode->HandleSeamlessTravelPlayer(TravelPlayer);
		}
	}
}

/// @cond DOXYGEN_WARNINGS

bool APlayerController::HasClientLoadedCurrentWorld()
{
	UNetConnection* Connection = Cast<UNetConnection>(Player);
	if (Connection == NULL && UNetConnection::GNetConnectionBeingCleanedUp != NULL && UNetConnection::GNetConnectionBeingCleanedUp->PlayerController == this)
	{
		Connection = UNetConnection::GNetConnectionBeingCleanedUp;
	}
	if (Connection != NULL)
	{
		// NOTE: To prevent exploits, child connections must not use the parent connections ClientWorldPackageName value at all.
		return (Connection->GetClientWorldPackageName() == GetWorld()->GetOutermost()->GetFName());
	}
	else
	{
		// if we have no client connection, we're local, so we always have the current world
		return true;
	}
}

void APlayerController::SmoothTargetViewRotation(APawn* TargetPawn, float DeltaSeconds)
{
	BlendedTargetViewRotation = FMath::RInterpTo(BlendedTargetViewRotation, TargetViewRotation, DeltaSeconds, SmoothTargetViewRotationSpeed);
}


void APlayerController::InitInputSystem()
{
	if (PlayerInput == nullptr)
	{
		const UClass* OverrideClass = OverridePlayerInputClass.Get();
		
		PlayerInput = NewObject<UPlayerInput>(this, OverrideClass ? OverrideClass : UInputSettings::GetDefaultPlayerInputClass());
	}

	SetupInputComponent();

	CurrentMouseCursor = DefaultMouseCursor;
	CurrentClickTraceChannel = DefaultClickTraceChannel;

	UWorld* World = GetWorld();
	check(World);
	World->PersistentLevel->PushPendingAutoReceiveInput(this);

	// setup optional touchscreen interface
	CreateTouchInterface();
}

void APlayerController::SafeRetryClientRestart()
{
	if (AcknowledgedPawn != GetPawn())
	{
		UWorld* World = GetWorld();
		check(World);

		if (World->TimeSince(LastRetryPlayerTime) > RetryClientRestartThrottleTime)
		{
			ClientRetryClientRestart(GetPawn());
			LastRetryPlayerTime = World->TimeSeconds;
		}
	}
}


/// @cond DOXYGEN_WARNINGS

/** Avoid calling ClientRestart if we have already accepted this pawn */
void APlayerController::ClientRetryClientRestart_Implementation(APawn* NewPawn)
{
	if (NewPawn == NULL)
	{
		return;
	}

	UE_LOGF(LogPlayerController, Verbose, "ClientRetryClientRestart_Implementation %ls, AcknowledgedPawn: %ls", *GetNameSafe(NewPawn), *GetNameSafe(AcknowledgedPawn));

	// Avoid calling ClientRestart if we have already accepted this pawn
	if( (GetPawn() != NewPawn) || (NewPawn->GetController() != this) || (NewPawn != AcknowledgedPawn) )
	{
		SetPawn(NewPawn);
		NewPawn->SetController(this);
		NewPawn->OnRep_Controller();
		ClientRestart(GetPawn());
	}
}

void APlayerController::ClientRestart_Implementation(APawn* NewPawn)
{
	UE_LOGF(LogPlayerController, Verbose, "ClientRestart_Implementation %ls", *GetNameSafe(NewPawn));

	ResetIgnoreInputFlags();
	AcknowledgedPawn = NULL;

	SetPawn(NewPawn);
	if ( (GetPawn() != NULL) && GetPawn()->GetTearOff() )
	{
		UnPossess();
		SetPawn(NULL);
		AcknowledgePossession(GetPawn());
		return;
	}

	if ( GetPawn() == NULL )
	{
		// We failed to possess, ask server to verify and potentially resend the pawn
		ServerCheckClientPossessionReliable();
		return;
	}

	// Only acknowledge non-null Pawns here. ClientRestart is only ever called by the Server for valid pawns,
	// but we may receive the function call before Pawn is replicated over, so it will resolve to NULL.
	AcknowledgePossession(GetPawn());

	AController* OldController = GetPawn()->GetController();
	GetPawn()->SetController(this);
	if (OldController != this)
	{
		// In case this is received before APawn::OnRep_Controller is called
		if (UE::Gameplay::CVars::bAlwaysNotifyClientOnControllerChange)
		{
			// When not in backward compatibility mode, OnRep_Controller will properly call NotifyControllerChanged
			GetPawn()->OnRep_Controller();
		}
		else
		{
			GetPawn()->NotifyControllerChanged();
		}
	}
	GetPawn()->DispatchRestart(true);
	
	if (GetLocalRole() < ROLE_Authority)
	{
		ChangeState( NAME_Playing );
		if (bAutoManageActiveCameraTarget)
		{
			AutoManageActiveCameraTarget(GetPawn());
			ResetCameraMode();
		}
	}
}

/// @endcond

void APlayerController::OnPossess(APawn* PawnToPossess)
{
	if ( PawnToPossess != NULL && 
		(PlayerState == NULL || !PlayerState->IsOnlyASpectator()) )
	{
		const bool bNewPawn = (GetPawn() != PawnToPossess);

		if (GetPawn() && bNewPawn)
		{
			UnPossess();
		}

		if (PawnToPossess->GetController() != NULL)
		{
			PawnToPossess->GetController()->UnPossess();
		}

		PawnToPossess->PossessedBy(this);

		// update rotation to match possessed pawn's rotation
		SetControlRotation( PawnToPossess->GetActorRotation() );

		SetPawn(PawnToPossess);
		check(GetPawn() != NULL);

		if (GetPawn() && GetPawn()->PrimaryActorTick.bStartWithTickEnabled)
		{
			GetPawn()->SetActorTickEnabled(true);
		}

		INetworkPredictionInterface* NetworkPredictionInterface = GetPawn() ? Cast<INetworkPredictionInterface>(GetPawn()->GetMovementComponent()) : NULL;
		if (NetworkPredictionInterface)
		{
			NetworkPredictionInterface->ResetPredictionData_Server();
		}

		AcknowledgedPawn = NULL;
		
		// Local PCs will have the Restart() triggered right away in ClientRestart (via PawnClientRestart()), but the server should call Restart() locally for remote PCs.
		// We're really just trying to avoid calling Restart() multiple times.
		if (!IsLocalPlayerController())
		{
			GetPawn()->DispatchRestart(false);
		}

		ClientRestart(GetPawn());
		
		ChangeState( NAME_Playing );
		if (bAutoManageActiveCameraTarget)
		{
			AutoManageActiveCameraTarget(GetPawn());
			ResetCameraMode();
		}
	}
}

void APlayerController::AcknowledgePossession(APawn* P)
{
	if (Cast<ULocalPlayer>(Player) != NULL)
	{
		AcknowledgedPawn = P;
		if (P != NULL)
		{
			P->RecalculateBaseEyeHeight();
		}
		ServerAcknowledgePossession(P);
	}
}

void APlayerController::ReceivedPlayer()
{
	if (IsInState(NAME_Spectating))
	{
		if (GetSpectatorPawn() == NULL)
		{
			BeginSpectatingState();
		}
	}

	if (Player)
	{
		Player->ReceivedPlayerController(this);
	}
}

FVector APlayerController::GetFocalLocation() const
{
	if (GetPawnOrSpectator())
	{
		return GetPawnOrSpectator()->GetActorLocation();
	}
	else
	{
		return GetSpawnLocation();
	}
}

void APlayerController::PostLoad()
{
	Super::PostLoad();

	if (GetLinkerUEVersion() < VER_UE4_SPLIT_TOUCH_AND_CLICK_ENABLES)
	{
		bEnableTouchEvents = bEnableClickEvents;
	}
}

void APlayerController::GetActorEyesViewPoint( FVector& out_Location, FRotator& out_Rotation ) const
{
	// If we have a Pawn, this is our view point.
	if (GetPawnOrSpectator() != NULL)
	{
		GetPawnOrSpectator()->GetActorEyesViewPoint(out_Location, out_Rotation);
	}
	else
	{
		out_Location = PlayerCameraManager ? PlayerCameraManager->GetCameraLocation() : GetSpawnLocation();
		out_Rotation = GetControlRotation();
	}
}

void APlayerController::CalcCamera(float DeltaTime, FMinimalViewInfo& OutResult)
{
	OutResult.Location = GetFocalLocation();
	OutResult.Rotation = GetControlRotation();
}


void APlayerController::GetPlayerViewPoint( FVector& out_Location, FRotator& out_Rotation ) const
{
	if (IsInState(NAME_Spectating) && HasAuthority() && !IsLocalController())
	{
		// Server uses the synced location from clients. Important for view relevancy checks.
		out_Location = LastSpectatorSyncLocation;
		out_Rotation = LastSpectatorSyncRotation;
	}
	else if (PlayerCameraManager != NULL && 
		PlayerCameraManager->GetCameraCacheTime() > 0.f) // Whether camera was updated at least once)
	{
		PlayerCameraManager->GetCameraViewPoint(out_Location, out_Rotation);
	}
	else
	{
		AActor* TheViewTarget = GetViewTarget();

		if( TheViewTarget != NULL )
		{
			out_Location = TheViewTarget->GetActorLocation();
			out_Rotation = TheViewTarget->GetActorRotation();
		}
		else
		{
			Super::GetPlayerViewPoint(out_Location,out_Rotation);
		}

		out_Location.DiagnosticCheckNaN(*FString::Printf(TEXT("APlayerController::GetPlayerViewPoint: out_Location, ViewTarget=%s"), *GetNameSafe(TheViewTarget)));
		out_Rotation.DiagnosticCheckNaN(*FString::Printf(TEXT("APlayerController::GetPlayerViewPoint: out_Rotation, ViewTarget=%s"), *GetNameSafe(TheViewTarget)));
	}
}

void APlayerController::UpdateRotation( float DeltaTime )
{
	// Calculate Delta to be applied on ViewRotation
	FRotator DeltaRot(RotationInput);

	FRotator ViewRotation = GetControlRotation();

	if (PlayerCameraManager)
	{
		PlayerCameraManager->ProcessViewRotation(DeltaTime, ViewRotation, DeltaRot);
	}

	AActor* ViewTarget = GetViewTarget();
	if (!PlayerCameraManager || !ViewTarget || !ViewTarget->HasActiveCameraComponent() || ViewTarget->HasActivePawnControlCameraComponent())
	{
		if (IsLocalPlayerController() && GEngine->XRSystem.IsValid() && GetWorld() != nullptr && GEngine->XRSystem->IsHeadTrackingAllowedForWorld(*GetWorld()))
		{
			auto XRCamera = GEngine->XRSystem->GetXRCamera();
			if (XRCamera.IsValid())
			{
				XRCamera->ApplyHMDRotation(this, ViewRotation);
			}
		}
	}

	SetControlRotation(ViewRotation);

	APawn* const P = GetPawnOrSpectator();
	if (P)
	{
		P->FaceRotation(ViewRotation, DeltaTime);
	}
}

void APlayerController::FellOutOfWorld(const UDamageType& dmgType) {}

void APlayerController::PostInitializeComponents()
{
	Super::PostInitializeComponents();

	if (IsValidChecked(this) && (GetNetMode() != NM_Client) )
	{
		// create a new player replication info
		InitPlayerState();
	}

	SpawnPlayerCameraManager();
	ResetCameraMode(); 

	if ( GetNetMode() == NM_Client )
	{
		SpawnDefaultHUD();
	}

	AddCheats();

	bPlayerIsWaiting = true;
	StateName = NAME_Spectating; // Don't use ChangeState, because we want to defer spawning the SpectatorPawn until the Player is received
}

/// @cond DOXYGEN_WARNINGS

bool APlayerController::ServerShortTimeout_Validate()
{
	return true;
}

void APlayerController::ServerShortTimeout_Implementation()
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_PC_ServerShortTimeout);
	if (!bShortConnectTimeOut)
	{
		UWorld* World = GetWorld();
		check(World);

		bShortConnectTimeOut = true;

		// quick update of pickups and gameobjectives since this player is now relevant
		if (GetWorldSettings()->GetPauserPlayerState() != NULL)
		{
			// update everything immediately, as TimeSeconds won't get advanced while paused
			// so otherwise it won't happen at all until the game is unpaused
			// this floods the network, but we're paused, so no gameplay is going on that would care much
			for (TSharedPtr<FNetworkObjectInfo> NetworkObjectInfo : World->GetNetDriver()->GetNetworkObjectList().GetAllObjects())
			{
				if (NetworkObjectInfo.IsValid())
				{
					AActor* const A = NetworkObjectInfo->WeakActor.Get();
					if (A)
					{
						if (!A->bOnlyRelevantToOwner)
						{
							A->ForceNetUpdate();
						}
					}
				}
			}
		}
		else 
		{
			if (World->GetNetDriver())
			{
				float NetUpdateTimeOffset = (World->GetAuthGameMode()->GetNumPlayers() < 8) ? 0.2f : 0.5f;
				auto ValidActorTest = [](const AActor* const Actor)
				{
					return (Actor->GetNetUpdateFrequency() < 1) && !Actor->bOnlyRelevantToOwner;
				};
				World->GetNetDriver()->ForceAllActorsNetUpdateTime(NetUpdateTimeOffset, ValidActorTest);
			}
		}
	}
}

/// @cond DOXYGEN_WARNINGS

void APlayerController::AddCheats(bool bForce)
{
	// Cheat manager is completely disabled in shipping by default
#if UE_WITH_CHEAT_MANAGER
	UWorld* World = GetWorld();
	check(World);

	// Abort if cheat manager exists or there is no cheat class
	if (CheatManager || !CheatClass)
	{
		return;
	}

	// Spawn if game mode says we are allowed, or if bForce
	if ( (World->GetAuthGameMode() && World->GetAuthGameMode()->AllowCheats(this)) || bForce)
	{
		CheatManager = NewObject<UCheatManager>(this, CheatClass);
		CheatManager->InitCheatManager();
	}
#endif
}

void APlayerController::EnableCheats()
{
	// In non-shipping builds this can be called to enable cheats in multiplayer and override AllowCheats
#if !UE_BUILD_SHIPPING
	AddCheats(true);
#else
	AddCheats();
#endif
}

void APlayerController::SpawnDefaultHUD()
{
	if ( Cast<ULocalPlayer>(Player) == NULL )
	{
		return;
	}

	UE_LOGF(LogPlayerController, Verbose, "SpawnDefaultHUD");
	FActorSpawnParameters SpawnInfo;
	SpawnInfo.Owner = this;
	SpawnInfo.Instigator = GetInstigator();
	SpawnInfo.ObjectFlags |= RF_Transient;	// We never want to save HUDs into a map
	MyHUD = GetWorld()->SpawnActor<AHUD>( SpawnInfo );
}

void APlayerController::CreateTouchInterface()
{
	ULocalPlayer* LocalPlayer = Cast<ULocalPlayer>(Player);

	// do we want to show virtual joysticks?
	if (LocalPlayer && LocalPlayer->ViewportClient && SVirtualJoystick::ShouldDisplayTouchInterface())
	{
		// in case we already had one, remove it
		if (VirtualJoystick.IsValid())
		{
			Cast<ULocalPlayer>(Player)->ViewportClient->RemoveViewportWidgetContent(VirtualJoystick.ToSharedRef());
		}

		if (CurrentTouchInterface == nullptr)
		{
			// load what the game wants to show at startup
			FSoftObjectPath DefaultTouchInterfaceName = GetDefault<UInputSettings>()->DefaultTouchInterface;

			if (DefaultTouchInterfaceName.IsValid())
			{
				// activate this interface if we have it
				CurrentTouchInterface = LoadObject<UTouchInterface>(NULL, *DefaultTouchInterfaceName.ToString());
			}
		}

		if (CurrentTouchInterface)
		{
			// create the joystick 
			VirtualJoystick = CreateVirtualJoystick();

			// add it to the player's viewport
			LocalPlayer->ViewportClient->AddViewportWidgetContent(VirtualJoystick.ToSharedRef());

			ActivateTouchInterface(CurrentTouchInterface);
		}
	}
}

TSharedPtr<SVirtualJoystick> APlayerController::CreateVirtualJoystick()
{
	return SNew(SVirtualJoystick);
}

void APlayerController::CleanupGameViewport()
{
	ULocalPlayer* LocalPlayer = Cast<ULocalPlayer>(Player);

	if (VirtualJoystick.IsValid())
	{
		ActivateTouchInterface(nullptr);
	}
}

AHUD* APlayerController::GetHUD() const
{
	return MyHUD;
}

void APlayerController::SetMouseCursorWidget(EMouseCursor::Type Cursor, class UUserWidget* CursorWidget)
{
	ULocalPlayer* LocalPlayer = Cast<ULocalPlayer>(Player);
	if (LocalPlayer && LocalPlayer->ViewportClient)
	{
		LocalPlayer->ViewportClient->SetSoftwareCursorWidget(Cursor, CursorWidget);
	}
}

void APlayerController::GetViewportSize(int32& SizeX, int32& SizeY) const
{
	SizeX = 0;
	SizeY = 0;

	ULocalPlayer* LocPlayer = Cast<ULocalPlayer>(Player);
	if( LocPlayer && LocPlayer->ViewportClient )
	{
		FVector2D ViewportSize;
		LocPlayer->ViewportClient->GetViewportSize(ViewportSize);

		SizeX = ViewportSize.X;
		SizeY = ViewportSize.Y;
	}
}

void APlayerController::Reset()
{
	if ( GetPawn() != NULL )
	{
		PawnPendingDestroy( GetPawn() );
		UnPossess();
	}

	Super::Reset();

	SetViewTarget(this);
	ResetCameraMode();

	bPlayerIsWaiting = !PlayerState->IsOnlyASpectator();
	ChangeState(NAME_Spectating);
}

/// @cond DOXYGEN_WARNINGS

void APlayerController::ClientReset_Implementation()
{
	ResetCameraMode();
	SetViewTarget(this);

	bPlayerIsWaiting = (PlayerState == nullptr) || !PlayerState->IsOnlyASpectator();
	ChangeState(NAME_Spectating);
}

void APlayerController::ClientGotoState_Implementation(FName NewState)
{
	ChangeState(NewState);
}

/// @endcond


void APlayerController::UnFreeze() {}

bool APlayerController::IsFrozen()
{
	return GetWorldTimerManager().IsTimerActive(TimerHandle_UnFreeze);
}

/// @cond DOXYGEN_WARNINGS

void APlayerController::ServerAcknowledgePossession_Implementation(APawn* P)
{
	UE_LOGF(LogPlayerController, Verbose, "ServerAcknowledgePossession_Implementation %ls", *GetNameSafe(P));
	AcknowledgedPawn = P;

	if (UE::Gameplay::CVars::NetResetServerPredictionDataOnPawnAck != 0)
	{
		if (AcknowledgedPawn && AcknowledgedPawn == GetPawn())
		{
			INetworkPredictionInterface* NetworkPredictionInterface = GetPawn() ? Cast<INetworkPredictionInterface>(GetPawn()->GetMovementComponent()) : NULL;
			if (NetworkPredictionInterface)
			{
				NetworkPredictionInterface->ResetPredictionData_Server();
			}
		}
	}
}

bool APlayerController::ServerAcknowledgePossession_Validate(APawn* P)
{
	if (P)
	{
		// Valid to acknowledge no possessed pawn
		RPC_VALIDATE( !P->HasAnyFlags(RF_ClassDefaultObject) );
	}
	return true;
}

/// @endcond

void APlayerController::OnUnPossess()
{
	if (GetPawn() != NULL)
	{
		if (GetLocalRole() == ROLE_Authority)
		{
			GetPawn()->SetReplicates(true);
		}
		GetPawn()->UnPossessed();

		if (GetViewTarget() == GetPawn())
		{
			SetViewTarget(this);
		}
	}
	SetPawn(NULL);
}

/// @cond DOXYGEN_WARNINGS

void APlayerController::ClientSetHUD_Implementation(TSubclassOf<AHUD> NewHUDClass)
{
	if ( MyHUD != NULL )
	{

		MyHUD->Destroy();
		MyHUD = NULL;
	}

	FActorSpawnParameters SpawnInfo;
	SpawnInfo.Owner = this;
	SpawnInfo.Instigator = GetInstigator();
	SpawnInfo.ObjectFlags |= RF_Transient;	// We never want to save HUDs into a map

	MyHUD = GetWorld()->SpawnActor<AHUD>(NewHUDClass, SpawnInfo );
}

/// @endcond

void APlayerController::CleanupPlayerState()
{
	if (PlayerState)
	{
		// By default this destroys it, but games can override
		PlayerState->OnDeactivated();
	}
	SetPlayerState(NULL);
}

void APlayerController::OnActorChannelOpen(FInBunch& InBunch, UNetConnection* Connection)
{
	SetAsLocalPlayerController();

	// Attempt to match the player controller to a local viewport (client side)
	InBunch << NetPlayerIndex;

	if (InBunch.EngineNetVer() >= FEngineNetworkCustomVersion::ClientHandshakeId)
	{
		InBunch << ClientHandshakeId;
	}

	if (InBunch.EngineNetVer() >= FEngineNetworkCustomVersion::CloseChildConnection)
	{
		InBunch << LocalPlayerConnectionIdentifier;
	}
	
	if (Connection->Driver != NULL && Connection == Connection->Driver->ServerConnection)
	{
		const bool bIsParentConnection = 
			(NetPlayerIndex == 0) ||
			(GetGameInstance() && GetGameInstance()->GetNumLocalPlayers() == 1);

		if (bIsParentConnection)
		{
			// main connection PlayerController
			Connection->HandleClientPlayer(this, Connection); 

			// sets the reserved bandwidth on the bandwidth manager for gameplay in Kbps
			FDownlinkBandwidthManager& DBManager = FDownlinkBandwidthManager::Get();
			DBManager.SetGameplayBandwidthCap(Connection->Driver->MaxInternetClientRate / UE::BandwidthManager::KilobitsToBytesConversionValue);
		}
		else
		{
			// Only create a new child connection if there isn't alreayd one associated with a NetPlayerIndex.
			UChildConnection* ChildConnection = Connection->GetChildConnectionWithNetPlayerIndex(NetPlayerIndex);
			if (!ChildConnection)
			{
				ChildConnection = Connection->CreateChildWithNetPlayerIndex(NetPlayerIndex);
			}

			if (ensure(ChildConnection))
			{
				UE_LOGF(LogNet, Log, "[%ls:%ls] Binding player controller %ls to child connection %ls", 
					   *GetNameSafe(Connection->GetDriver()), *GetNameSafe(Connection), *GetName(), *GetNameSafe(ChildConnection));

				ChildConnection->HandleClientPlayer(this, Connection);
			}
		}
	}
}

bool APlayerController::UseShortConnectTimeout() const
{
	return bShortConnectTimeOut;
}

void APlayerController::OnSerializeNewActor(FOutBunch& OutBunch)
{
	// serialize PlayerIndex as part of the initial bunch for PlayerControllers so they can be matched to the correct client-side viewport
	OutBunch << NetPlayerIndex;
	OutBunch << ClientHandshakeId;
	OutBunch << LocalPlayerConnectionIdentifier;
}

void APlayerController::OnNetCleanup(UNetConnection* Connection)
{
	UWorld* World = GetWorld();
	// destroy the PC that was waiting for a swap, if it exists
	if (World != NULL)
	{
		World->DestroySwappedPC(Connection);
	}

	check(UNetConnection::GNetConnectionBeingCleanedUp == NULL);
	UNetConnection::GNetConnectionBeingCleanedUp = Connection;
	//@note: if we ever implement support for splitscreen players leaving a match without the primary player leaving, we'll need to insert
	// a call to ClearOnlineDelegates() here so that PlayerController.ClearOnlineDelegates can use the correct ControllerId (which lives
	// in ULocalPlayer)
	if (Player && Player->PlayerController == this)
	{
		Player->PlayerController = nullptr;
	}
	Player = NULL;
	NetConnection = NULL;	
	Destroy( true );
	UNetConnection::GNetConnectionBeingCleanedUp = NULL;
}

/// @cond DOXYGEN_WARNINGS

void APlayerController::ClientReceiveLocalizedMessage_Implementation( TSubclassOf<ULocalMessage> Message, int32 Switch, APlayerState* RelatedPlayerState_1, APlayerState* RelatedPlayerState_2, UObject* OptionalObject )
{
	// Wait for player to be up to date with replication when joining a server, before stacking up messages
	if (GetNetMode() == NM_DedicatedServer || GetWorld()->GetGameState() == nullptr || Message == nullptr)
	{
		return;
	}

	FClientReceiveData ClientData;
	ClientData.LocalPC = this;
	ClientData.MessageIndex = Switch;
	ClientData.RelatedPlayerState_1 = RelatedPlayerState_1;
	ClientData.RelatedPlayerState_2 = RelatedPlayerState_2;
	ClientData.OptionalObject = OptionalObject;

	Message->GetDefaultObject<ULocalMessage>()->ClientReceive( ClientData );
}

void APlayerController::ClientPlaySound_Implementation(USoundBase* Sound, float VolumeMultiplier /* = 1.f */, float PitchMultiplier /* = 1.f */)
{
	FVector AudioPosition = GetFocalLocation();
	UGameplayStatics::PlaySoundAtLocation( this, Sound, AudioPosition, VolumeMultiplier, PitchMultiplier );
}

void APlayerController::ClientPlaySoundAtLocation_Implementation(USoundBase* Sound, FVector Location, float VolumeMultiplier /* = 1.f */, float PitchMultiplier /* = 1.f */)
{
	UGameplayStatics::PlaySoundAtLocation( this, Sound, Location, VolumeMultiplier, PitchMultiplier );
}

void APlayerController::ClientMessage_Implementation( const FString& S, FName Type, float MsgLifeTime )
{
	if ( GetNetMode() == NM_DedicatedServer || GetWorld()->GetGameState() == nullptr )
	{
		return;
	}

	if (Type == NAME_None)
	{
		Type = FName("Event");
	}

	ClientTeamMessage(PlayerState, S, Type, MsgLifeTime);
}

void APlayerController::ClientTeamMessage_Implementation( APlayerState* SenderPlayerState, const FString& S, FName Type, float MsgLifeTime  )
{
	FString SMod = S;
	static FName NAME_Say = FName("Say");
	if( (Type == NAME_Say) && ( SenderPlayerState != NULL ) )
	{
		SMod = FString::Printf(TEXT("%s: %s"), *SenderPlayerState->GetPlayerName(), *SMod);
	}

	// since this is on the client, we can assume that if Player exists, it is a LocalPlayer
	if (Player != NULL)
	{
		UGameViewportClient *ViewportClient = CastChecked<ULocalPlayer>(Player)->ViewportClient;
		if ( ViewportClient && ViewportClient->ViewportConsole )
		{
			CastChecked<ULocalPlayer>(Player)->ViewportClient->ViewportConsole->OutputText(SMod);
		}
	}
}

bool APlayerController::ServerToggleAILogging_Validate()
{
	return true;
}

void APlayerController::ServerToggleAILogging_Implementation()
{
	if (CheatManager)
	{
		CheatManager->ServerToggleAILogging();
	}
}

/// @endcond

void APlayerController::PawnLeavingGame()
{
	if (GetPawn() != NULL)
	{
		GetPawn()->Destroy();
		SetPawn(NULL);
	}
}

void APlayerController::BeginPlay()
{
	Super::BeginPlay();

	// If the viewport is currently set to lock mouse always, we need to cache what widget the mouse needs to be locked to even if the
	// widget does not have mouse capture.
	ULocalPlayer* LocalPlayer = Cast<ULocalPlayer>( Player );

	if ( LocalPlayer && LocalPlayer->ViewportClient )
	{
		if ( LocalPlayer->ViewportClient->ShouldAlwaysLockMouse() )
		{
			LocalPlayer->GetSlateOperations().LockMouseToWidget( LocalPlayer->ViewportClient->GetGameViewportWidget().ToSharedRef() );
		}
	}

	//If we are faking touch events show the cursor
	if (FSlateApplication::IsInitialized() && FSlateApplication::Get().IsFakingTouchEvents())
	{
		SetShowMouseCursor(true);
	}
}

void APlayerController::EndPlay(const EEndPlayReason::Type EndPlayReason)
{
	ULocalPlayer* LocalPlayer = Cast<ULocalPlayer>(Player);
	if (LocalPlayer)
	{
		if (VirtualJoystick.IsValid())
		{
			ActivateTouchInterface(nullptr);
		}

		if (FSlateApplication::IsInitialized())
		{
			IInputInterface* InputInterface = FSlateApplication::Get().GetInputInterface();
			if (InputInterface)
			{
				// Stop any force feedback effects that may be active
				InputInterface->SetForceFeedbackChannelValues(LocalPlayer->GetControllerId(), FForceFeedbackValues());
			}
		}
	}

	if (CheatManager)
	{
		CheatManager->ReceiveEndPlay();
	}
	
	Super::EndPlay(EndPlayReason);
}

void APlayerController::Destroyed()
{
	if (GetPawn() != NULL)
	{
		// Handle players leaving the game
		if (Player == NULL && GetLocalRole() == ROLE_Authority)
		{
			PawnLeavingGame();
		}
		else
		{
			UnPossess();
		}
	}

	if (GetSpectatorPawn() != NULL)
	{
		DestroySpectatorPawn();
	}
	if ( MyHUD != NULL )
	{
		MyHUD->Destroy();
		MyHUD = NULL;
	}

	if (PlayerCameraManager != NULL)
	{
		PlayerCameraManager->Destroy();
		PlayerCameraManager = NULL;
	}

	// Tells the game info to forcibly remove this player's CanUnpause delegates from its list of Pausers.
	// Prevents the game from being stuck in a paused state when a PC that paused the game is destroyed before the game is unpaused.
	AGameModeBase* const GameMode = GetWorld()->GetAuthGameMode();
	if (GameMode)
	{
		GameMode->ForceClearUnpauseDelegates(this);
	}

	PlayerInput = NULL;
	CheatManager = NULL;

	Super::Destroyed();
}

void APlayerController::FOV(float F)
{
	if (PlayerCameraManager != NULL)
	{
		PlayerCameraManager->SetFOV(F);
	}
}

void APlayerController::PreClientTravel( const FString& PendingURL, ETravelType TravelType, bool bIsSeamlessTravel )
{
	UGameInstance* GameInstance = GetGameInstance();
	if (GameInstance)
	{
		GameInstance->NotifyPreClientTravel(PendingURL, TravelType, bIsSeamlessTravel);
	}
}

void APlayerController::Camera( FName NewMode )
{
	ServerCamera(NewMode);
}

/// @cond DOXYGEN_WARNINGS

void APlayerController::ServerCamera_Implementation( FName NewMode )
{
	SetCameraMode(NewMode);
}

bool APlayerController::ServerCamera_Validate( FName NewMode )
{
	RPC_VALIDATE( NewMode.IsValid() );
	return true;
}

void APlayerController::ClientSetCameraMode_Implementation( FName NewCamMode )
{
	if (PlayerCameraManager)
	{
		PlayerCameraManager->CameraStyle = NewCamMode;
	}
}

/// @endcond

void APlayerController::SetCameraMode( FName NewCamMode )
{
	if (PlayerCameraManager)
	{
		PlayerCameraManager->CameraStyle = NewCamMode;
	}
	
	if ( GetNetMode() == NM_DedicatedServer )
	{
		ClientSetCameraMode( NewCamMode );
	}
}

void APlayerController::ResetCameraMode()
{
	FName DefaultMode = NAME_Default;
	if (PlayerCameraManager)
	{
		DefaultMode = PlayerCameraManager->CameraStyle;
	}

	SetCameraMode(DefaultMode);
}

/// @cond DOXYGEN_WARNINGS

void APlayerController::ClientSetCameraFade_Implementation(bool bEnableFading, FColor FadeColor, FVector2D FadeAlpha, float FadeTime, bool bFadeAudio, bool bHoldWhenFinished)
{
	if (PlayerCameraManager != nullptr)
	{
		if (bEnableFading)
		{
			// Allow fading from the current FadeAmount to allow for smooth transitions into new fades
			const float FadeStart = FadeAlpha.X >= 0.f ? FadeAlpha.X : PlayerCameraManager->FadeAmount;
			PlayerCameraManager->StartCameraFade(FadeStart, FadeAlpha.Y, FadeTime, FadeColor.ReinterpretAsLinear(), bFadeAudio, bHoldWhenFinished);
		}
		else
		{
			PlayerCameraManager->StopCameraFade();
		}
	}
}

/// @endcond

void APlayerController::SendClientAdjustment()
{
	// Server sends updates.
	// Note: we do this for both the pawn and spectator in case an implementation has a networked spectator.
	APawn* RemotePawn = GetPawnOrSpectator();
	if (RemotePawn && (RemotePawn->GetRemoteRole() == ROLE_AutonomousProxy) && !IsNetMode(NM_Client))
	{
		INetworkPredictionInterface* NetworkPredictionInterface = Cast<INetworkPredictionInterface>(RemotePawn->GetMovementComponent());
		if (NetworkPredictionInterface)
		{
			NetworkPredictionInterface->SendClientAdjustment();
		}
	}
}

/// @cond DOXYGEN_WARNINGS

void APlayerController::ClientCapBandwidth_Implementation(int32 Cap)
{
	ClientCap = Cap;
	if( (Player != NULL) && (Player->CurrentNetSpeed > Cap) )
	{
		SetNetSpeed(Cap);
	}
}

/// @endcond

void APlayerController::UpdatePing(float InPing)
{
	if (PlayerState != NULL)
	{
		PlayerState->UpdatePing(InPing);
	}
}


void APlayerController::SetSpawnLocation(const FVector& NewLocation)
{
	LastSpectatorSyncLocation = NewLocation;

	if (UE::Gameplay::CVars::bIsPlayerControllerPushBased)
	{
		COMPARE_ASSIGN_AND_MARK_PROPERTY_DIRTY(APlayerController, SpawnLocation, NewLocation, this);
		return;
	}
	
	SpawnLocation = NewLocation;
}


void APlayerController::SetInitialLocationAndRotation(const FVector& NewLocation, const FRotator& NewRotation)
{
	Super::SetInitialLocationAndRotation(NewLocation, NewRotation);
	SetSpawnLocation(NewLocation);
	if (GetSpectatorPawn())
	{
		GetSpectatorPawn()->TeleportTo(NewLocation, NewRotation, false, true);
	}
}

/// @cond DOXYGEN_WARNINGS

bool APlayerController::ServerUpdateCamera_Validate(FVector_NetQuantize CamLoc, int32 CamPitchAndYaw)
{
	return true;
}

void APlayerController::ServerUpdateCamera_Implementation(FVector_NetQuantize CamLoc, int32 CamPitchAndYaw)
{
	if (!PlayerCameraManager || !PlayerCameraManager->bUseClientSideCameraUpdates)
	{
		return;
	}

	FPOV NewPOV;
	NewPOV.Location = FRepMovement::RebaseOntoLocalOrigin(CamLoc, this);
	
	NewPOV.Rotation.Yaw = FRotator::DecompressAxisFromShort( (CamPitchAndYaw >> 16) & 65535 );
	NewPOV.Rotation.Pitch = FRotator::DecompressAxisFromShort(CamPitchAndYaw & 65535);

#if ENABLE_DRAW_DEBUG
	if ( PlayerCameraManager->bDebugClientSideCamera )
	{
		// show differences (on server) between local and replicated camera
		const FVector PlayerCameraLoc = PlayerCameraManager->GetCameraLocation();

		UWorld* World = GetWorld();
		DrawDebugSphere(World, PlayerCameraLoc, 10, 10, FColor::Green );
		DrawDebugSphere(World, NewPOV.Location, 10, 10, FColor::Yellow );
		DrawDebugLine(World, PlayerCameraLoc, PlayerCameraLoc + 100*PlayerCameraManager->GetCameraRotation().Vector(), FColor::Green);
		DrawDebugLine(World, NewPOV.Location, NewPOV.Location + 100*NewPOV.Rotation.Vector(), FColor::Yellow);
	}
	else
#endif
	{
		//@TODO: CAMERA: Fat pipe
		FMinimalViewInfo NewInfo = PlayerCameraManager->GetCameraCacheView();
		NewInfo.Location = NewPOV.Location;
		NewInfo.Rotation = NewPOV.Rotation;
		PlayerCameraManager->FillCameraCache(NewInfo);
	}
}

/// @endcond

bool APlayerController::ServerExecRPC_Validate(const FString& Msg)
{
	return true;
}

void APlayerController::ServerExecRPC_Implementation(const FString& Msg)
{
#if !UE_BUILD_SHIPPING
	ClientMessage(ConsoleCommand(Msg));
#endif
}
	
void APlayerController::ServerExec(const FString& Msg)
{
#if !UE_BUILD_SHIPPING

	if (Msg.Len() > 128)
	{
		UE_LOGF(LogPlayerController, Warning, "APlayerController::ServerExec. Msg too big for network RPC. Truncating to 128 character");
	}

	ServerExecRPC(Msg.Left(128));
#endif
}

void APlayerController::RestartLevel()
{
	if( GetNetMode()==NM_Standalone )
	{
		ClientTravel( TEXT("?restart"), TRAVEL_Relative );
	}
}

void APlayerController::LocalTravel( const FString& FURL )
{
	if( GetNetMode()==NM_Standalone )
	{
		ClientTravel( FURL, TRAVEL_Relative );
	}
}

void APlayerController::ClientReturnToMainMenuWithTextReason_Implementation(const FText& ReturnReason)
{
	if (UGameInstance* const GameInstance = GetGameInstance())
	{
		GameInstance->ReturnToMainMenu();
	}
	else
	{
		UWorld* const World = GetWorld();
		GEngine->HandleDisconnect(World, World->GetNetDriver());
	}
}

bool APlayerController::SetPause( bool bPause, FCanUnpause CanUnpauseDelegate)
{
	bool bResult = false;
	if (GetNetMode() != NM_Client)
	{
		AGameModeBase* const GameMode = GetWorld()->GetAuthGameMode();
		if (GameMode != nullptr)
		{
			bool bCurrentPauseState = IsPaused();
			if (bPause && !bCurrentPauseState)
			{
				// Pause gamepad rumbling too if needed
				bResult = GameMode->SetPause(this, CanUnpauseDelegate);

				if (AWorldSettings* WorldSettings = GetWorldSettings())
				{
					WorldSettings->ForceNetUpdate();
				}
			}
			else if (!bPause && bCurrentPauseState)
			{
				bResult = GameMode->ClearPause();
			}
		}
	}
	return bResult;
}

bool APlayerController::IsPaused() const
{
	return GetWorldSettings()->GetPauserPlayerState() != NULL;
}

void APlayerController::Pause()
{
	ServerPause();
}

/// @cond DOXYGEN_WARNINGS

bool APlayerController::ServerPause_Validate()
{
#if UE_BUILD_SHIPPING
	// Don't let clients remotely pause the game in shipping builds.
	return IsLocalController();
#else
	return true;
#endif
}

void APlayerController::ServerPause_Implementation()
{
	SetPause(!IsPaused());
}

/// @endcond

void APlayerController::SetName(const FString& S)
{
	if (!S.IsEmpty())
	{
		// Games can override this to persist name on the client if desired
		ServerChangeName(S);
	}
}

/// @cond DOXYGEN_WARNINGS

void APlayerController::ServerChangeName_Implementation( const FString& S )
{
	AGameModeBase* GameMode = GetWorld()->GetAuthGameMode();
	if (!S.IsEmpty() && GameMode)
	{
		GameMode->ChangeName( this, S, true );
	}
}

bool APlayerController::ServerChangeName_Validate( const FString& S )
{
	RPC_VALIDATE( !S.IsEmpty() );
	return true;
}

/// @endcond

void APlayerController::SwitchLevel(const FString& FURL)
{
	const ENetMode NetMode = GetNetMode();
	if (NetMode == NM_Standalone || NetMode == NM_ListenServer)
	{
		GetWorld()->ServerTravel(FURL);
	}
}

void APlayerController::NotifyLoadedWorld(FName WorldPackageName, bool bFinalDest)
{
	// place the camera at the first playerstart we can find
	SetViewTarget(this);
	
	if (TActorIterator<APlayerStart> It(GetWorld()); It)
	{
		APlayerStart* P = *It;

		FRotator SpawnRotation(ForceInit);
		SpawnRotation.Yaw = P->GetActorRotation().Yaw;
		SetInitialLocationAndRotation(P->GetActorLocation(), SpawnRotation);
	}
}

void APlayerController::GameHasEnded(AActor* EndGameFocus, bool bIsWinner)
{
	// and transition to the game ended state
	SetViewTarget(EndGameFocus);
	ClientGameEnded(EndGameFocus, bIsWinner);
}

/// @cond DOXYGEN_WARNINGS

void APlayerController::ClientGameEnded_Implementation(AActor* EndGameFocus, bool bIsWinner)
{
	SetViewTarget(EndGameFocus);
}

/// @endcond

bool APlayerController::GetHitResultUnderCursor(ECollisionChannel TraceChannel, bool bTraceComplex, FHitResult& HitResult) const
{
	ULocalPlayer* LocalPlayer = Cast<ULocalPlayer>(Player);
	bool bHit = false;
	if (LocalPlayer && LocalPlayer->ViewportClient)
	{
		FVector2D MousePosition;
		if (LocalPlayer->ViewportClient->GetMousePosition(MousePosition))
		{
			bHit = GetHitResultAtScreenPosition(MousePosition, TraceChannel, bTraceComplex, HitResult);
		}
	}

	if(!bHit)	//If there was no hit we reset the results. This is redundant but helps Blueprint users
	{
		HitResult = FHitResult();
	}

	return bHit;
}

bool APlayerController::GetHitResultUnderCursorByChannel(ETraceTypeQuery TraceChannel, bool bTraceComplex, FHitResult& HitResult) const
{
	ULocalPlayer* LocalPlayer = Cast<ULocalPlayer>(Player);
	bool bHit = false;
	if (LocalPlayer && LocalPlayer->ViewportClient)
	{
		FVector2D MousePosition;
		if (LocalPlayer->ViewportClient->GetMousePosition(MousePosition))
		{
			bHit = GetHitResultAtScreenPosition(MousePosition, TraceChannel, bTraceComplex, HitResult);
		}
	}

	if(!bHit)	//If there was no hit we reset the results. This is redundant but helps Blueprint users
	{
		HitResult = FHitResult();
	}

	return bHit;
}

bool APlayerController::GetHitResultUnderCursorForObjects(const TArray<TEnumAsByte<EObjectTypeQuery> > & ObjectTypes, bool bTraceComplex, FHitResult& HitResult) const
{
	ULocalPlayer* LocalPlayer = Cast<ULocalPlayer>(Player);
	bool bHit = false;
	if (LocalPlayer && LocalPlayer->ViewportClient)
	{
		FVector2D MousePosition;
		if (LocalPlayer->ViewportClient->GetMousePosition(MousePosition))
		{
			bHit = GetHitResultAtScreenPosition(MousePosition, ObjectTypes, bTraceComplex, HitResult);
		}
	}

	if(!bHit)	//If there was no hit we reset the results. This is redundant but helps Blueprint users
	{
		HitResult = FHitResult();
	}

	return bHit;
}

bool APlayerController::GetHitResultUnderFinger(ETouchIndex::Type FingerIndex, ECollisionChannel TraceChannel, bool bTraceComplex, FHitResult& HitResult) const
{
	bool bHit = false;
	if (PlayerInput)
	{
		FVector2f TouchPosition;
		bool bIsPressed = false;
		GetInputTouchState(FingerIndex, TouchPosition.X, TouchPosition.Y, bIsPressed);
		if (bIsPressed)
		{
			bHit = GetHitResultAtScreenPosition(FVector2D(TouchPosition), TraceChannel, bTraceComplex, HitResult);
		}
	}

	if(!bHit)	//If there was no hit we reset the results. This is redundant but helps Blueprint users
	{
		HitResult = FHitResult();
	}

	return bHit;
}

bool APlayerController::GetHitResultUnderFingerByChannel(ETouchIndex::Type FingerIndex, ETraceTypeQuery TraceChannel, bool bTraceComplex, FHitResult& HitResult) const
{
	bool bHit = false;
	if (PlayerInput)
	{
		FVector2f TouchPosition;
		bool bIsPressed = false;
		GetInputTouchState(FingerIndex, TouchPosition.X, TouchPosition.Y, bIsPressed);
		if (bIsPressed)
		{
			bHit = GetHitResultAtScreenPosition(FVector2D(TouchPosition), TraceChannel, bTraceComplex, HitResult);
		}
	}

	if(!bHit)	//If there was no hit we reset the results. This is redundant but helps Blueprint users
	{
		HitResult = FHitResult();
	}

	return bHit;
}

bool APlayerController::GetHitResultUnderFingerForObjects(ETouchIndex::Type FingerIndex, const TArray<TEnumAsByte<EObjectTypeQuery> > & ObjectTypes, bool bTraceComplex, FHitResult& HitResult) const
{
	bool bHit = false;
	if (PlayerInput)
	{
		FVector2f TouchPosition;
		bool bIsPressed = false;
		GetInputTouchState(FingerIndex, TouchPosition.X, TouchPosition.Y, bIsPressed);
		if (bIsPressed)
		{
			bHit = GetHitResultAtScreenPosition(FVector2D(TouchPosition), ObjectTypes, bTraceComplex, HitResult);
		}
	}

	if(!bHit)	//If there was no hit we reset the results. This is redundant but helps Blueprint users
	{
		HitResult = FHitResult();
	}

	return bHit;
}

bool APlayerController::DeprojectMousePositionToWorld(FVector& WorldLocation, FVector& WorldDirection) const
{
	ULocalPlayer* const LocalPlayer = GetLocalPlayer();
	if (LocalPlayer && LocalPlayer->ViewportClient)
	{
		FVector2D ScreenPosition;
		if (LocalPlayer->ViewportClient->GetMousePosition(ScreenPosition))
		{
			return UGameplayStatics::DeprojectScreenToWorld(this, ScreenPosition, WorldLocation, WorldDirection);
		}
	}

	return false;
}

bool APlayerController::DeprojectScreenPositionToWorld(float ScreenX, float ScreenY, FVector& WorldLocation, FVector& WorldDirection) const
{
	return UGameplayStatics::DeprojectScreenToWorld(this, FVector2D(ScreenX, ScreenY), WorldLocation, WorldDirection);
}


bool APlayerController::ProjectWorldLocationToScreen(FVector WorldLocation, FVector2D& ScreenLocation, bool bPlayerViewportRelative) const
{
	return UGameplayStatics::ProjectWorldToScreen(this, WorldLocation, ScreenLocation, bPlayerViewportRelative);
}

bool APlayerController::ProjectWorldLocationToScreenWithDistance(FVector WorldLocation, FVector& ScreenLocation, bool bPlayerViewportRelative) const
{
	// find distance
	ULocalPlayer const* const LP = GetLocalPlayer();
	if (LP && LP->ViewportClient)
	{
		// get the projection data
		FSceneViewProjectionData ProjectionData;
		if (LP->GetProjectionData(LP->ViewportClient->Viewport, /*out*/ ProjectionData))
		{
			FVector2D ScreenPosition2D;
			FMatrix const ViewProjectionMatrix = ProjectionData.ComputeViewProjectionMatrix();
			if ( FSceneView::ProjectWorldToScreen(WorldLocation, ProjectionData.GetConstrainedViewRect(), ViewProjectionMatrix, ScreenPosition2D) )
			{
				if ( bPlayerViewportRelative )
				{
					ScreenPosition2D -= FVector2D(ProjectionData.GetConstrainedViewRect().Min);
				}

				PostProcessWorldToScreen(WorldLocation, ScreenPosition2D, bPlayerViewportRelative);
				ScreenLocation = FVector(ScreenPosition2D.X, ScreenPosition2D.Y, FVector::Dist(ProjectionData.ViewOrigin, WorldLocation));

				return true;
			}
		}
	}

	return false;
}

bool APlayerController::PostProcessWorldToScreen(FVector WorldLocation, FVector2D& ScreenLocation, bool bPlayerViewportRelative) const
{
	return true;
}

bool APlayerController::GetHitResultAtScreenPosition(const FVector2D ScreenPosition, const ECollisionChannel TraceChannel, const FCollisionQueryParams& CollisionQueryParams, FHitResult& HitResult) const
{
	// Early out if we clicked on a HUD hitbox
	if (GetHUD() != NULL && GetHUD()->GetHitBoxAtCoordinates(ScreenPosition, true))
	{
		return false;
	}

	FVector WorldOrigin;
	FVector WorldDirection;
	if (UGameplayStatics::DeprojectScreenToWorld(this, ScreenPosition, WorldOrigin, WorldDirection) == true)
	{
		return GetWorld()->LineTraceSingleByChannel(HitResult, WorldOrigin, WorldOrigin + WorldDirection * HitResultTraceDistance, TraceChannel, CollisionQueryParams);
	}

	return false;
}

bool APlayerController::GetHitResultAtScreenPosition(const FVector2D ScreenPosition, const ECollisionChannel TraceChannel, bool bTraceComplex, FHitResult& HitResult) const
{
	FCollisionQueryParams CollisionQueryParams(SCENE_QUERY_STAT(ClickableTrace), bTraceComplex );
	return GetHitResultAtScreenPosition( ScreenPosition, TraceChannel, CollisionQueryParams, HitResult );
}

bool APlayerController::GetHitResultAtScreenPosition(const FVector2D ScreenPosition, const ETraceTypeQuery TraceChannel, bool bTraceComplex, FHitResult& HitResult) const
{
	return GetHitResultAtScreenPosition( ScreenPosition, UEngineTypes::ConvertToCollisionChannel( TraceChannel ), bTraceComplex, HitResult );
}

bool APlayerController::GetHitResultAtScreenPosition(const FVector2D ScreenPosition, const TArray<TEnumAsByte<EObjectTypeQuery> > & ObjectTypes, bool bTraceComplex, FHitResult& HitResult) const
{
	// Early out if we clicked on a HUD hitbox
	if (GetHUD() != NULL && GetHUD()->GetHitBoxAtCoordinates(ScreenPosition, true))
	{
		return false;
	}

	FVector WorldOrigin;
	FVector WorldDirection;
	if (UGameplayStatics::DeprojectScreenToWorld(this, ScreenPosition, WorldOrigin, WorldDirection) == true)
	{
		FCollisionObjectQueryParams const ObjParam(ObjectTypes);
		return GetWorld()->LineTraceSingleByObjectType(HitResult, WorldOrigin, WorldOrigin + WorldDirection * HitResultTraceDistance, ObjParam, FCollisionQueryParams(SCENE_QUERY_STAT(ClickableTrace), bTraceComplex));
	}

	return false;
}

void APlayerController::SetMouseLocation(const int X, const int Y)
{
	ULocalPlayer* LocalPlayer = Cast<ULocalPlayer>( Player );
	if (LocalPlayer)
	{
		UGameViewportClient* ViewportClient = LocalPlayer->ViewportClient;
		if (ViewportClient)
		{
			FViewport* Viewport = ViewportClient->Viewport;
			if (Viewport)
			{
				Viewport->SetMouse( X, Y );
			}
		}
	}
}

/* PlayerTick is only called if the PlayerController has a PlayerInput object.  Therefore, it will not be called on servers for non-locally controlled playercontrollers. */
void APlayerController::PlayerTick( float DeltaTime )
{
#if WITH_CHAOS_VISUAL_DEBUGGER

	if (bNetworkPhysicsTickOffsetAssigned)
	{
		CVD_TRACE_NETWORK_TICK_OFFSET(NetworkPhysicsTickOffset, CVD_TRACE_GET_SOLVER_ID_FROM_WORLD(GetWorld()));
	}

#endif

	if (!bShortConnectTimeOut)
	{
		bShortConnectTimeOut = true;
		ServerShortTimeout();
	}

	TickPlayerInput(DeltaTime, DeltaTime == 0.f);

	if ((Player != NULL) && (Player->PlayerController == this))
	{
		// Validate current state
		bool bUpdateRotation = false;
		if (IsInState(NAME_Playing))
		{
			if( GetPawn() == NULL )
			{
				ChangeState(NAME_Inactive);
			}
			else if (Player && GetPawn() && GetPawn() == AcknowledgedPawn)
			{
				bUpdateRotation = true;
			}
		}
		
		if ( IsInState(NAME_Inactive) )
		{
			if (GetLocalRole() < ROLE_Authority)
			{
				SafeServerCheckClientPossession();
			}

			bUpdateRotation = !IsFrozen();
		}
		else if ( IsInState(NAME_Spectating) )
		{
			if (GetLocalRole() < ROLE_Authority)
			{
				SafeServerUpdateSpectatorState();
			}

			bUpdateRotation = true;
		}

		// Update rotation
		if (bUpdateRotation)
		{
			UpdateRotation(DeltaTime);
		}
	}
}

void APlayerController::FlushPressedKeys()
{
	if (PlayerInput)
	{
		PlayerInput->FlushPressedKeys();
	}
}

TSubclassOf<UPlayerInput> APlayerController::GetOverridePlayerInputClass() const
{
	return OverridePlayerInputClass;
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
bool APlayerController::InputKey(const FInputKeyParams& Params)
{
	if (Params.Delta.Y != 0.0 || Params.Delta.Z != 0.0)
	{
		UE_LOGF(LogPlayerInput, Warning, "Call to the deprecated version of UPlayerInput::InputKey will no longer consider the YZ components of the input delta. Use paired key axis instead.");
	}
	
	FInputKeyEventArgs NewArgs(
		/*Viewport*/ nullptr,
		Params.InputDevice,
		Params.Key,
		/*Delta*/Params.Delta.X,
		Params.DeltaTime,
		Params.NumSamples,
		/*timestamp*/0u);

	NewArgs.Event = Params.Event;
	
	return InputKey(NewArgs);
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

bool APlayerController::InputKey(const FInputKeyEventArgs& Params)
{
	bool bResult = false;

	// Only process the given input if it came from an input device that is owned by our owning local player
	if (GetDefault<UInputSettings>()->bFilterInputByPlatformUser &&
		IPlatformInputDeviceMapper::Get().GetUserForInputDevice(Params.InputDevice) != GetPlatformUserId())
	{
		return false;
	}
	
	// Any analog values can simply be passed to the UPlayerInput
	if(Params.Key.IsAnalog())
	{
		if(PlayerInput)
		{
			bResult = PlayerInput->InputKey(Params);
		}
	}
	else
	{
		if (PlayerInput)
		{
			bResult = PlayerInput->InputKey(Params);
			if (bEnableClickEvents && (ClickEventKeys.Contains(Params.Key) || ClickEventKeys.Contains(EKeys::AnyKey)))
			{
				FVector2D MousePosition;
				UGameViewportClient* ViewportClient = CastChecked<ULocalPlayer>(Player)->ViewportClient;
				if (ViewportClient && ViewportClient->GetMousePosition(MousePosition))
				{
					UPrimitiveComponent* ClickedPrimitive = nullptr;
					if (bEnableMouseOverEvents)
					{
						ClickedPrimitive = CurrentClickablePrimitive.Get();
					}
					else
					{
						FHitResult HitResult;
						const bool bHit = GetHitResultAtScreenPosition(MousePosition, CurrentClickTraceChannel, true, HitResult);
						if (bHit)
						{
							ClickedPrimitive = HitResult.Component.Get();
						}
					}
					if(GetHUD())
					{
						if (GetHUD()->UpdateAndDispatchHitBoxClickEvents(MousePosition, Params.Event))
						{
							ClickedPrimitive = nullptr;
						}
					}

					if (ClickedPrimitive)
					{
						switch(Params.Event)
						{
						case IE_Pressed:
						case IE_DoubleClick:
							ClickedPrimitive->DispatchOnClicked(Params.Key);
							break;

						case IE_Released:
							ClickedPrimitive->DispatchOnReleased(Params.Key);
							break;

						case IE_Axis:
						case IE_Repeat:
							break;
						}
					}

					bResult = true;
				}
			}
		}
	}
	
	return bResult;
}

bool APlayerController::InputTouch(
	const FInputDeviceId DeviceId,
	const uint32 Handle,
	const ETouchType::Type Type,
	const FVector2D& TouchLocation,
	const float Force,
	const uint32 TouchpadIndex,
	const uint64 Timestamp)
{
	return InputTouch(FTouchId(DeviceId, static_cast<ETouchIndex::Type>(Handle)), Type, TouchLocation, Force, Timestamp);
}

bool APlayerController::InputTouch(const FTouchId TouchId, const ETouchType::Type Type, const FVector2D& TouchLocation, const float Force,
	const uint64 Timestamp)
{
	bool bResult = false;
	if (PlayerInput)
	{
		bResult = PlayerInput->InputTouch(TouchId, Type, TouchLocation, Force, Timestamp);

		if (bEnableTouchEvents || bEnableTouchOverEvents)
		{
			FHitResult HitResult;
			const bool bHit = GetHitResultAtScreenPosition(TouchLocation, CurrentClickTraceChannel, true, HitResult);

			ProcessTouchHitResult(TouchId, Type, TouchLocation, HitResult, bHit);
		}
	}

	return bResult;
}

void APlayerController::ProcessTouchHitResult(const FInputDeviceId DeviceId, uint32 Handle, ETouchType::Type Type, const FVector2D& TouchLocation,
	const FHitResult& HitResult, bool bHit)
{
	ProcessTouchHitResult(FTouchId(DeviceId, static_cast<ETouchIndex::Type>(Handle)), Type, TouchLocation, HitResult, bHit);
}

void APlayerController::ProcessTouchHitResult(const FTouchId TouchId, const ETouchType::Type Type, const FVector2D& TouchLocation,
	const FHitResult& HitResult, bool bHit)
{
	if (GetHUD())
	{
		if (Type == ETouchType::Began || Type == ETouchType::Ended)
		{
			if (GetHUD()->UpdateAndDispatchHitBoxClickEvents(TouchLocation, (Type == ETouchType::Began ? EInputEvent::IE_Pressed : EInputEvent::IE_Released)))
			{
				bHit = false;
			}
		}
	}

	ETouchEventFamilies EnabledFamilies = ETouchEventFamilies::None;
	if (bEnableTouchEvents)
	{
		EnabledFamilies |= ETouchEventFamilies::BeginEnd;
	}
	if (bEnableTouchOverEvents)
	{
		EnabledFamilies |= ETouchEventFamilies::Over;
	}
	if (EnabledFamilies != ETouchEventFamilies::None)
	{
		UPrimitiveComponent::DispatchTouchHitResult(TouchId, Type, HitResult, bHit, this, EnabledFamilies, CurrentTouchablePrimitives);
	}
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
bool APlayerController::InputMotion(const FVector& Tilt, const FVector& RotationRate, const FVector& Gravity, const FVector& Acceleration)
{
	// Use the primary input device id for this player controller. The deprecated code path doesnt have any other way to grab it
	// and this is a sensible default
	const FInputDeviceId DeviceId = IPlatformInputDeviceMapper::Get().GetPrimaryInputDeviceForUser(GetPlatformUserId());

	// A timestamp of 0 is invalid because we don't have any way to accurately get one here from deprecated callsites.
	constexpr uint64 Timestamp = 0u;

	return InputMotion(DeviceId, Tilt, RotationRate, Gravity, Acceleration, Timestamp);
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

bool APlayerController::InputMotion(const FInputDeviceId DeviceId, const FVector& Tilt, const FVector& RotationRate, const FVector& Gravity, const FVector& Acceleration, const uint64 Timestamp)
{
	bool bResult = false;

	if (PlayerInput && bEnableMotionControls)
	{
		bResult = PlayerInput->InputMotion(DeviceId, Tilt, RotationRate, Gravity, Acceleration, Timestamp);
	}

	return bResult;
}

bool APlayerController::InputGesture(const FInputDeviceId DeviceId, EGestureEvent GestureType, EGesturePhase GesturePhase, const FVector2D& GestureDelta, bool bIsDirectionInvertedFromDevice, const uint64 Timestamp)
{
	bool bResult = false;

	if (PlayerInput)
	{
		FKey GestureKey;
		switch (GestureType)
		{
		case EGestureEvent::Pan:      GestureKey = EKeys::Gesture_Pan; break;
		case EGestureEvent::Magnify:  GestureKey = EKeys::Gesture_Pinch; break;
		case EGestureEvent::Rotate:   GestureKey = EKeys::Gesture_Rotate; break;
		case EGestureEvent::Tap:      GestureKey = EKeys::Gesture_Tap; break;
		case EGestureEvent::LongPress: GestureKey = EKeys::Gesture_LongPress; break;
		case EGestureEvent::Flick:    GestureKey = EKeys::Gesture_Flick; break;
		default: return false;
		}

		EInputEvent InputEvent;
		switch (GesturePhase)
		{
		case EGesturePhase::Started: InputEvent = IE_Pressed; break;
		case EGesturePhase::Update:  InputEvent = IE_Repeat; break;
		case EGesturePhase::Ended:   InputEvent = IE_Released; break;
		default: return false;
		}
		bResult = PlayerInput->InputGesture(FInputKeyEventArgs(DeviceId, GestureKey, InputEvent, GestureDelta, Timestamp));
	}

	return bResult;
}

void APlayerController::SetMotionControlsEnabled(bool bEnabled)
{
	bEnableMotionControls = bEnabled;
	if (bEnableMotionControls && !GetDefault<UInputSettings>()->bEnableMotionControls)
	{
		UE_LOGF(LogPlayerController, Warning, "Player bEnableMotionControls has been set to true, but motion controls are disabled in the input settings! See UInputSettings::bEnableMotionControls");
	}
}

bool APlayerController::ShouldShowMouseCursor() const
{
	return bShowMouseCursor;
}

void APlayerController::SetShowMouseCursor(bool bShow)
{
	if (bShowMouseCursor != bShow)
	{
		UE_LOGF(LogViewport, Display, "Player bShowMouseCursor Changed, %ls -> %ls",
			bShowMouseCursor ? TEXT("True") : TEXT("False"),
			bShow ? TEXT("True") : TEXT("False")
		);

		bShowMouseCursor = bShow;
	}
}

EMouseCursor::Type APlayerController::GetMouseCursor() const
{
	if (ShouldShowMouseCursor())
	{
		return CurrentMouseCursor;
	}

	return EMouseCursor::None;
}

void APlayerController::SetupInputComponent()
{
	// A subclass could create a different InputComponent class but still want the default bindings
	if (InputComponent == NULL)
	{
		InputComponent = NewObject<UInputComponent>(this, UInputSettings::GetDefaultInputComponentClass(), TEXT("PC_InputComponent0"));
		InputComponent->RegisterComponent();
	}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
	if (UInputDelegateBinding::SupportsInputDelegate(GetClass()))
PRAGMA_ENABLE_DEPRECATION_WARNINGS		
	{
		InputComponent->bBlockInput = bBlockInput;		
		UInputDelegateBinding::BindInputDelegatesWithSubojects(this, InputComponent);
	}
}


void APlayerController::BuildInputStack(TArray<UInputComponent*>& InputStack)
{
	// Controlled pawn gets last dibs on the input stack
	APawn* ControlledPawn = GetPawnOrSpectator();
	if (ControlledPawn)
	{
		if (ControlledPawn->InputEnabled())
		{
			// Get the explicit input component that is created upon Pawn possession. This one gets last dibs.
			if (ControlledPawn->InputComponent)
			{
				InputStack.Push(ControlledPawn->InputComponent);
			}

			// See if there is another InputComponent that was added to the Pawn's components array (possibly by script).
			for (UActorComponent* ActorComponent : ControlledPawn->GetComponents())
			{
				UInputComponent* PawnInputComponent = Cast<UInputComponent>(ActorComponent);

				// Do not add this input component it is already contained on the "CurrentInputStack". Input components which are on the CurrentInputStack
				// already, will have been specifically added via the PushInputComponent function already. Adding them again here would result in them being
				// processed twice
				if (PawnInputComponent && PawnInputComponent != ControlledPawn->InputComponent && !CurrentInputStack.Contains(PawnInputComponent))
				{
					InputStack.Push(PawnInputComponent);
				}
			}
		}
	}

	// LevelScriptActors are put on the stack next
	for (ULevel* Level : GetWorld()->GetLevels())
	{
		ALevelScriptActor* ScriptActor = Level->GetLevelScriptActor();
		if (ScriptActor)
		{
			if (ScriptActor->InputEnabled() && ScriptActor->InputComponent)
			{
				InputStack.Push(ScriptActor->InputComponent);
			}
		}
	}

	if (InputEnabled())
	{
		InputStack.Push(InputComponent);
	}

	// Components pushed on to the stack get priority
	for (int32 Idx=0; Idx<CurrentInputStack.Num(); ++Idx)
	{
		UInputComponent* IC = CurrentInputStack[Idx].Get();
		if (IsValid(IC))
		{
			InputStack.Push(IC);
		}
		else
		{
			CurrentInputStack.RemoveAt(Idx--);
		}
	}

	// We need to sort the input stack in order to respect the priority of each input component.
	// This way, all input components and their priorities will have their order respected. Not just those
	// manually added via "PushInputComponent"
	if (UE::Gameplay::CVars::bSortInputStackByPriority)
	{
		Algo::StableSort(InputStack, [](const UInputComponent* A, const UInputComponent* B)
			{
				return A->Priority < B->Priority;
			});	
	}
}

void APlayerController::ProcessPlayerInput(const float DeltaTime, const bool bGamePaused)
{
	static TArray<UInputComponent*> InputStack;

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	LastUsedInputStack.Reset();
#endif
	
	// must be called non-recursively and on the game thread
	check(IsInGameThread() && !InputStack.Num());

	// process all input components in the stack, top down
	{
		SCOPE_CYCLE_COUNTER(STAT_PC_BuildInputStack);
		BuildInputStack(InputStack);
	}

	// process the desired components
	{
		SCOPE_CYCLE_COUNTER(STAT_PC_ProcessInputStack);
		PlayerInput->ProcessInputStack(InputStack, DeltaTime, bGamePaused);
	}

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	{
		for (const UInputComponent* IC : InputStack)
		{
			LastUsedInputStack.Add(IC);
		}
	}
#endif
	
	InputStack.Reset();
}

void APlayerController::PreProcessInput(const float DeltaTime, const bool bGamePaused)
{
}

void APlayerController::PostProcessInput(const float DeltaTime, const bool bGamePaused)
{
	if( IsLookInputIgnored() )
	{
		// zero look inputs
		RotationInput = FRotator::ZeroRotator;
	}
}

void APlayerController::ResetIgnoreInputFlags()
{
	// The movement locks can be set in cinematic mode, but if a restart occurs, we don't want them to be reset.
	if (!bCinemaDisableInputMove)
	{
		IgnoreMoveInput = GetDefault<APlayerController>()->IgnoreMoveInput;
	}

	if (!bCinemaDisableInputLook)
	{
		IgnoreLookInput = GetDefault<APlayerController>()->IgnoreLookInput;
	}
}

void APlayerController::SetCinematicMode( bool bInCinematicMode, bool bAffectsMovement, bool bAffectsTurning)
{
	if ( bAffectsMovement && (bInCinematicMode != bCinemaDisableInputMove) )
	{
		SetIgnoreMoveInput(bInCinematicMode);
		bCinemaDisableInputMove = bInCinematicMode;
	}
	if ( bAffectsTurning && (bInCinematicMode != bCinemaDisableInputLook) )
	{
		SetIgnoreLookInput(bInCinematicMode);
		bCinemaDisableInputLook = bInCinematicMode;
	}
}


void APlayerController::SetViewTargetWithBlend(AActor* NewViewTarget, float BlendTime, EViewTargetBlendFunction BlendFunc, float BlendExp, bool bLockOutgoing)
{
	FViewTargetTransitionParams TransitionParams;
	TransitionParams.BlendTime = BlendTime;
	TransitionParams.BlendFunction = BlendFunc;
	TransitionParams.BlendExp = BlendExp;
	TransitionParams.bLockOutgoing = bLockOutgoing;

	SetViewTarget(NewViewTarget, TransitionParams);
}

/// @cond DOXYGEN_WARNINGS

void APlayerController::ClientSetViewTarget_Implementation( AActor* A, FViewTargetTransitionParams TransitionParams )
{
	if (PlayerCameraManager && !PlayerCameraManager->bClientSimulatingViewTarget)
	{
		if( A == NULL )
		{
			ServerVerifyViewTarget();
			return;
		}
		// don't force view to self while unpossessed (since server may be doing it having destroyed the pawn)
		if ( IsInState(NAME_Inactive) && A == this )
		{
			return;
		}
		SetViewTarget(A, TransitionParams);
	}
}

bool APlayerController::ServerVerifyViewTarget_Validate()
{
	return true;
}

void APlayerController::ServerVerifyViewTarget_Implementation()
{
	AActor* TheViewTarget = GetViewTarget();
	if( TheViewTarget == this )
	{
		return;
	}
	ClientSetViewTarget( TheViewTarget );
}

/// @endcond

void APlayerController::SpawnPlayerCameraManager()
{
	// servers and owning clients get cameras
	// If no archetype specified, spawn an Engine.PlayerCameraManager.  NOTE all games should specify an archetype.
	FActorSpawnParameters SpawnInfo;
	SpawnInfo.Owner = this;
	SpawnInfo.Instigator = GetInstigator();
	SpawnInfo.ObjectFlags |= RF_Transient;	// We never want to save camera managers into a map
	if (PlayerCameraManagerClass != NULL)
	{
		PlayerCameraManager = GetWorld()->SpawnActor<APlayerCameraManager>(PlayerCameraManagerClass, SpawnInfo);
	}
	else
	{
		PlayerCameraManager = GetWorld()->SpawnActor<APlayerCameraManager>(SpawnInfo);
	}

	if (PlayerCameraManager != NULL)
	{
		PlayerCameraManager->InitializeFor(this);
	}
	else
	{
		UE_LOGF(LogPlayerController, Log,  "Couldn't Spawn PlayerCameraManager for Player!!" );
	}
}

void APlayerController::GetAudioListenerPosition(FVector& OutLocation, FVector& OutFrontDir, FVector& OutRightDir) const
{
	FVector ViewLocation;
	FRotator ViewRotation;

	if (bOverrideAudioListener)
	{
		USceneComponent* ListenerComponent = AudioListenerComponent.Get();
		if (ListenerComponent != nullptr)
		{
			ViewRotation = ListenerComponent->GetComponentRotation() + AudioListenerRotationOverride;
			ViewLocation = ListenerComponent->GetComponentLocation() + ViewRotation.RotateVector(AudioListenerLocationOverride);
		}
		else
		{
			ViewLocation = AudioListenerLocationOverride;
			ViewRotation = AudioListenerRotationOverride;
		}
	}
	else
	{
		GetPlayerViewPoint(ViewLocation, ViewRotation);
	}

	const FRotationTranslationMatrix ViewRotationMatrix(ViewRotation, ViewLocation);

	OutLocation = ViewLocation;
	OutFrontDir = ViewRotationMatrix.GetUnitAxis( EAxis::X );
	OutRightDir = ViewRotationMatrix.GetUnitAxis( EAxis::Y );
}

bool APlayerController::GetAudioListenerAttenuationOverridePosition(FVector& OutLocation) const
{
	if (bOverrideAudioAttenuationListener)
	{
		USceneComponent* ListenerComponent = AudioListenerAttenuationComponent.Get();
		if (ListenerComponent)
		{
			OutLocation = ListenerComponent->GetComponentLocation() + AudioListenerAttenuationOverride;
		}
		else
		{
			OutLocation = AudioListenerAttenuationOverride;
		}
		return true;
	}
	return false;
}


void APlayerController::SetAudioListenerOverride(USceneComponent* AttachedComponent, FVector Location, FRotator Rotation)
{
	bOverrideAudioListener = true;
	AudioListenerComponent = AttachedComponent;
	AudioListenerLocationOverride = Location;
	AudioListenerRotationOverride = Rotation;
}

void APlayerController::ClearAudioListenerOverride()
{
	bOverrideAudioListener = false;
	AudioListenerComponent = nullptr;
}

void APlayerController::SetAudioListenerAttenuationOverride(USceneComponent* AttachToComponent, FVector AttenuationLocationOverride)
{
	bOverrideAudioAttenuationListener = true;
	AudioListenerAttenuationComponent = AttachToComponent;
	AudioListenerAttenuationOverride = AttenuationLocationOverride;
}

void APlayerController::ClearAudioListenerAttenuationOverride()
{
	bOverrideAudioAttenuationListener = false;
	AudioListenerAttenuationComponent = nullptr;
}

/// @cond DOXYGEN_WARNINGS

bool APlayerController::ServerCheckClientPossession_Validate()
{
	return true;
}

void APlayerController::ServerCheckClientPossession_Implementation()
{
	if (AcknowledgedPawn != GetPawn())
	{
		// Client already throttles their call to this function, so respond immediately by resetting LastRetryClientTime
		LastRetryPlayerTime = ForceRetryClientRestartTime;
		SafeRetryClientRestart();			
	}
}

bool APlayerController::ServerCheckClientPossessionReliable_Validate()
{
	return true;
}

void APlayerController::ServerCheckClientPossessionReliable_Implementation()
{
	ServerCheckClientPossession_Implementation();
}

/// @endcond

void APlayerController::SafeServerCheckClientPossession()
{
	if (GetPawn() && AcknowledgedPawn != GetPawn())
	{
		UWorld* World = GetWorld();
		if (World->TimeSince(LastRetryPlayerTime) > RetryServerAcknowledgeThrottleTime)
		{
			ServerCheckClientPossession();
			LastRetryPlayerTime = World->TimeSeconds;
		}
	}
}

void APlayerController::SafeServerUpdateSpectatorState()
{
	if (IsInState(NAME_Spectating))
	{
		UWorld* World = GetWorld();
		if (World->TimeSince(LastSpectatorStateSynchTime) > RetryServerCheckSpectatorThrottleTime)
		{
			ServerSetSpectatorLocation(GetFocalLocation(), GetControlRotation());
			LastSpectatorStateSynchTime = World->TimeSeconds;
		}
	}
}

/// @cond DOXYGEN_WARNINGS

bool APlayerController::ServerSetSpectatorLocation_Validate(FVector NewLoc, FRotator NewRot)
{
	return true;
}

void APlayerController::ServerSetSpectatorLocation_Implementation(FVector NewLoc, FRotator NewRot)
{
	UWorld* World = GetWorld();
	if ( IsInState(NAME_Spectating) )
	{
		LastSpectatorSyncLocation = NewLoc;
		LastSpectatorSyncRotation = NewRot;
		if ( World->TimeSeconds - LastSpectatorStateSynchTime > 2.f )
		{
			ClientGotoState(GetStateName());
			LastSpectatorStateSynchTime = World->TimeSeconds;
		}
	}
	// if we receive this with !bIsSpectating, the client is in the wrong state; tell it what state it should be in
	else if (World->TimeSeconds != LastSpectatorStateSynchTime)
	{
		if (AcknowledgedPawn != GetPawn())
		{
			SafeRetryClientRestart();			
		}
		else
		{
			ClientGotoState(GetStateName());
			ClientSetViewTarget(GetViewTarget());
		}
		
		LastSpectatorStateSynchTime = World->TimeSeconds;
	}
}

bool APlayerController::ServerSetSpectatorWaiting_Validate(bool bWaiting)
{
	return true;
}

void APlayerController::ServerSetSpectatorWaiting_Implementation(bool bWaiting)
{
	if (IsInState(NAME_Spectating))
	{
		bPlayerIsWaiting = true;
	}
}

void APlayerController::ClientSetSpectatorWaiting_Implementation(bool bWaiting)
{
	if (IsInState(NAME_Spectating))
	{
		bPlayerIsWaiting = true;
	}
}

float APlayerController::GetDeprecatedInputYawScale() const
{
	if (GetDefault<UInputSettings>()->bEnableLegacyInputScales)
	{
		return InputYawScale_DEPRECATED;
	}
	else
	{
		UE_LOGF(LogPlayerController, Warning, "Attempting to access legacy input scales without the setting enabled! See UInputSettings::bEnableLegacyInputScales.");
		return 1.0f;
	}
}

float APlayerController::GetDeprecatedInputPitchScale() const
{
	if (GetDefault<UInputSettings>()->bEnableLegacyInputScales)
	{
		return InputPitchScale_DEPRECATED;
	}
	else
	{
		UE_LOGF(LogPlayerController, Warning, "Attempting to access legacy input scales without the setting enabled! See UInputSettings::bEnableLegacyInputScales.");
		return 1.0f;
	}
}

float APlayerController::GetDeprecatedInputRollScale() const
{
	if (GetDefault<UInputSettings>()->bEnableLegacyInputScales)
	{
		return InputRollScale_DEPRECATED;
	}
	else
	{
		UE_LOGF(LogPlayerController, Warning, "Attempting to access legacy input scales without the setting enabled! See UInputSettings::bEnableLegacyInputScales.");
		return 1.0f;
	}
}

void APlayerController::SetDeprecatedInputYawScale(float NewValue)
{
	if (GetDefault<UInputSettings>()->bEnableLegacyInputScales)
	{
		InputYawScale_DEPRECATED = NewValue;
	}
	else
	{
		UE_LOGF(LogPlayerController, Warning, "Attempting to access legacy input scales without the setting enabled! See UInputSettings::bEnableLegacyInputScales.");
	}
}

void APlayerController::SetDeprecatedInputPitchScale(float NewValue)
{
	if (GetDefault<UInputSettings>()->bEnableLegacyInputScales)
	{
		InputPitchScale_DEPRECATED = NewValue;
	}
	else
	{
		UE_LOGF(LogPlayerController, Warning, "Attempting to access legacy input scales without the setting enabled! See UInputSettings::bEnableLegacyInputScales.");
	}
}

void APlayerController::SetDeprecatedInputRollScale(float NewValue)
{
	if (GetDefault<UInputSettings>()->bEnableLegacyInputScales)
	{
		InputRollScale_DEPRECATED = NewValue;
	}
	else
	{
		UE_LOGF(LogPlayerController, Warning, "Attempting to access legacy input scales without the setting enabled! See UInputSettings::bEnableLegacyInputScales.");
	}
}

bool APlayerController::ServerViewNextPlayer_Validate()
{
	return true;
}

void APlayerController::ServerViewNextPlayer_Implementation()
{
	if (IsInState(NAME_Spectating))
	{
		ViewAPlayer(+1);
	}
}

bool APlayerController::ServerViewPrevPlayer_Validate()
{
	return true;
}

void APlayerController::ServerViewPrevPlayer_Implementation()
{
	if (IsInState(NAME_Spectating))
	{
		ViewAPlayer(-1);
	}
}

/// @endcond

APlayerState* APlayerController::GetNextViewablePlayer(int32 dir)
{
	UWorld* World = GetWorld();
	AGameModeBase* GameMode = World->GetAuthGameMode();
	AGameStateBase* GameState = World->GetGameState();

	// Can't continue unless we have the GameState and GameMode
	if (!GameState || !GameMode)
	{
		return nullptr;
	}

	APlayerState* NextPlayerState = (PlayerCameraManager ? PlayerCameraManager->ViewTarget.GetPlayerState() : nullptr);
	
	// If we don't have a NextPlayerState, use our own.
	// This will allow us to attempt to find another player to view or, if all else fails, makes sure we have a playerstate set for next time.
	int32 NextIndex = (NextPlayerState ? GameState->PlayerArray.Find(NextPlayerState) : GameState->PlayerArray.Find(PlayerState));

	//Check that NextIndex is a valid index, as Find() may return INDEX_NONE
	if (!GameState->PlayerArray.IsValidIndex(NextIndex))
	{
		return nullptr;
	}

	// Cycle through the player states until we find a valid one.
	for (int32 i = 0; i < GameState->PlayerArray.Num(); ++i)
	{
		NextIndex = ((NextIndex == 0) && (dir < 0)) ? (GameState->PlayerArray.Num() - 1) : ((NextIndex == (GameState->PlayerArray.Num() - 1)) && (dir > 0)) ? 0 : NextIndex += dir;
		NextPlayerState = GameState->PlayerArray[NextIndex];

		// Make sure we're not trying to view our own player state.
		if (NextPlayerState != PlayerState)
		{
			AController* NextController = Cast<AController>(NextPlayerState->GetOwner());

			// Check they have a pawn & the game mode is ok with us spectating them.
			if (NextController && NextController->GetPawn() && GameMode->CanSpectate(this, NextPlayerState))
			{
				break;
			}
		}
	}

	// If we've failed to find another player to view, we'll be back to our original view target playerstate.
	return NextPlayerState;
}

void APlayerController::ViewAPlayer(int32 dir)
{
	APlayerState* const NextPlayerState = GetNextViewablePlayer(dir);

	if ( NextPlayerState != nullptr )
	{
		SetViewTarget(NextPlayerState);
	}
}

/// @cond DOXYGEN_WARNINGS

bool APlayerController::ServerViewSelf_Validate(FViewTargetTransitionParams TransitionParams)
{
	return true;
}

void APlayerController::ServerViewSelf_Implementation(FViewTargetTransitionParams TransitionParams)
{
	if (IsInState(NAME_Spectating))
	{
		ResetCameraMode();
		SetViewTarget( this, TransitionParams );
		ClientSetViewTarget( this, TransitionParams );
	}
}

/// @endcond

void APlayerController::StartFire( uint8 FireModeNum ) 
{
	if ( ((IsInState(NAME_Spectating) && bPlayerIsWaiting) || IsInState(NAME_Inactive)) && !IsFrozen() )
	{
		ServerRestartPlayer();
	}
	else if ( IsInState(NAME_Spectating) )
	{
		ServerViewNextPlayer();
	}
	else if ( GetPawn() && !bCinematicMode && !GetWorld()->bPlayersOnly )
	{
		GetPawn()->PawnStartFire( FireModeNum );
	}
}

bool APlayerController::NotifyServerReceivedClientData(APawn* InPawn, float TimeStamp)
{
	if (GetPawn() != InPawn || (GetNetMode() == NM_Client))
	{
		return false;
	}

	if (AcknowledgedPawn != GetPawn())
	{
		SafeRetryClientRestart();
		return false;
	}

	return true;
}

/// @cond DOXYGEN_WARNINGS

bool APlayerController::ServerRestartPlayer_Validate()
{
	return true;
}

void APlayerController::ServerRestartPlayer_Implementation()
{
	UE_LOG(LogPlayerController, Verbose, TEXT("SERVER RESTART PLAYER"));
	if ( GetNetMode() == NM_Client )
	{
		return;
	}

	if ( IsInState(NAME_Inactive) || (IsInState(NAME_Spectating) && bPlayerIsWaiting) )
	{
		AGameModeBase* const GameMode = GetWorld()->GetAuthGameMode();

		// This can happen if you do something like delete a bunch of stuff at runtime in PIE or something like that.
		// We need to check here to prevent a crash
		if (!IsValid(GameMode))
		{
			UE_LOGF(LogPlayerController, Warning, "[APlayerController::ServerRestartPlayer_Implementation] Player Controller '%ls' requested restart but the game mode is null! Nothing will happen.", *GetNameSafe(this));
			return;
		}
		
		if ( !GameMode->PlayerCanRestart(this) )
		{
			return;
		}

		// If we're still attached to a Pawn, leave it
		if ( GetPawn() != NULL )
		{
			UnPossess();
		}

		GameMode->RestartPlayer(this);
	}
	else if ( GetPawn() != NULL )
	{
		ClientRetryClientRestart(GetPawn());
	}
}

/// @endcond

bool APlayerController::CanRestartPlayer()
{
	return PlayerState && !PlayerState->IsOnlyASpectator() && HasClientLoadedCurrentWorld() && PendingSwapConnection == NULL;
}

/// @cond DOXYGEN_WARNINGS

void APlayerController::ClientIgnoreMoveInput_Implementation(bool bIgnore)
{
	SetIgnoreMoveInput(bIgnore);
}

void APlayerController::ClientIgnoreLookInput_Implementation(bool bIgnore)
{
	SetIgnoreLookInput(bIgnore);
}

/// @endcond

void APlayerController::DisplayDebug(class UCanvas* Canvas, const FDebugDisplayInfo& DebugDisplay, float& YL, float& YPos)
{
	Super::DisplayDebug(Canvas, DebugDisplay, YL, YPos);

	FDisplayDebugManager& DisplayDebugManager = Canvas->DisplayDebugManager;
	DisplayDebugManager.SetDrawColor(FColor(255, 255, 0));
	DisplayDebugManager.DrawString(FString::Printf(TEXT("STATE %s"), *GetStateName().ToString()));

	if (DebugDisplay.IsDisplayOn(NAME_Camera))
	{
		if (PlayerCameraManager != nullptr)
		{
			DisplayDebugManager.DrawString(FString("<<<< CAMERA >>>>"));
			PlayerCameraManager->DisplayDebug(Canvas, DebugDisplay, YL, YPos);
		}
		else
		{
			DisplayDebugManager.SetDrawColor(FColor::Red);
			DisplayDebugManager.DrawString(FString("<<<< NO CAMERA >>>>"));
		}
	}
	if ( DebugDisplay.IsDisplayOn(NAME_Input) )
	{
		TArray<UInputComponent*> InputStack;
		BuildInputStack(InputStack);

		DisplayDebugManager.SetDrawColor(FColor::White);
		DisplayDebugManager.DrawString(FString("<<<< INPUT STACK >>>"));

		for(int32 i=InputStack.Num() - 1; i >= 0; --i)
		{
			AActor* InputComponentOwner = InputStack[i]->GetOwner();
			DisplayDebugManager.SetDrawColor(FColor::White);
			if (InputComponentOwner)
			{
				DisplayDebugManager.DrawString(FString::Printf(TEXT(" %s.%s"), *InputComponentOwner->GetName(), *InputStack[i]->GetName()));
			}
			else
			{
				DisplayDebugManager.DrawString(FString::Printf(TEXT(" %s"), *InputStack[i]->GetName()));
			}
		}

		if (PlayerInput)
		{
			PlayerInput->DisplayDebug(Canvas, DebugDisplay, YL, YPos);
		}
		else
		{
			DisplayDebugManager.SetDrawColor(FColor::Red);
			DisplayDebugManager.DrawString(FString("NO INPUT"));
		}
	}
	if ( DebugDisplay.IsDisplayOn("ForceFeedback"))
	{
		DisplayDebugManager.SetDrawColor(FColor::White);
		DisplayDebugManager.DrawString(FString::Printf(TEXT("Force Feedback - Enabled: %s LL: %.2f LS: %.2f RL: %.2f RS: %.2f"), (bForceFeedbackEnabled ? TEXT("true") : TEXT("false")), ForceFeedbackValues.LeftLarge, ForceFeedbackValues.LeftSmall, ForceFeedbackValues.RightLarge, ForceFeedbackValues.RightSmall));
		DisplayDebugManager.DrawString(FString::Printf(TEXT("Pawn: %s"), this->AcknowledgedPawn ? *this->AcknowledgedPawn->GetFName().ToString() : TEXT("none")));
		
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
		DisplayDebugManager.DrawString(TEXT("-------------Last Played Force Feedback--------------"));												      
		DisplayDebugManager.DrawString(TEXT("Name Tag Duration IsLooping StartTime"));
		const float CurrentTime = GetWorld()->GetTimeSeconds();
		for (int32 i = ForceFeedbackEffectHistoryEntries.Num() - 1; i >= 0; --i)
		{
			if (CurrentTime > ForceFeedbackEffectHistoryEntries[i].TimeShown + 5.0f)
			{
				ForceFeedbackEffectHistoryEntries.RemoveAtSwap(i, EAllowShrinking::No);
			}
			else
			{
				const FActiveForceFeedbackEffect& LastActiveEffect = ForceFeedbackEffectHistoryEntries[i].LastActiveForceFeedbackEffect;
				const FString HistoryEntry = FString::Printf(TEXT("%s %s %f %s %f"), 
															*LastActiveEffect.ForceFeedbackEffect->GetFName().ToString(), 
															*LastActiveEffect.Parameters.Tag.ToString(), 
															LastActiveEffect.ForceFeedbackEffect->GetDuration(),
															(LastActiveEffect.Parameters.bLooping ? TEXT("true") : TEXT("false")),
															ForceFeedbackEffectHistoryEntries[i].TimeShown);
				DisplayDebugManager.DrawString(HistoryEntry);
			}
		}
		DisplayDebugManager.DrawString(TEXT("-----------------------------------------------------"));

		DisplayDebugManager.DrawString(TEXT("----------Current Playing Force Feedback-------------"));
		DisplayDebugManager.DrawString(TEXT("Name Tag/Component Distance Duration IsLooping PlayTime"));
		for (int32 Index = ActiveForceFeedbackEffects.Num() - 1; Index >= 0; --Index)
		{
			const FActiveForceFeedbackEffect& ActiveEffect = ActiveForceFeedbackEffects[Index];
			FForceFeedbackValues ActiveValues;
			ActiveEffect.GetValues(ActiveValues);
			if (ActiveValues.LeftLarge > 0.f || ActiveValues.LeftSmall > 0.f || ActiveValues.RightLarge > 0.f || ActiveValues.RightSmall > 0.f)
			{
				const FString ActiveEntry = FString::Printf(TEXT("%s %s N/A %.2f %s %.2f - LL: %.2f LS: %.2f RL: %.2f RS: %.2f"),
					*ActiveEffect.ForceFeedbackEffect->GetFName().ToString(),
					*ActiveEffect.Parameters.Tag.ToString(),
					ActiveEffect.ForceFeedbackEffect->GetDuration(),
					(ActiveEffect.Parameters.bLooping ? TEXT("true") : TEXT("false")),
					ActiveEffect.PlayTime,
					ActiveValues.LeftLarge, ActiveValues.LeftSmall, ActiveValues.RightLarge, ActiveValues.RightSmall);
				DisplayDebugManager.DrawString(ActiveEntry);
			}
		}
		if (FForceFeedbackManager* FFM = FForceFeedbackManager::Get(GetWorld()))
		{
			FFM->DrawDebug(GetFocalLocation(), DisplayDebugManager, GetPlatformUserId());
		}
		DisplayDebugManager.DrawString(TEXT("-----------------------------------------------------"));
#endif
	}

	YPos = DisplayDebugManager.GetYPos();
}

void APlayerController::SetCinematicMode(bool bInCinematicMode, bool bHidePlayer, bool bAffectsHUD, bool bAffectsMovement, bool bAffectsTurning)
{
	bCinematicMode = bInCinematicMode;
	bHidePawnInCinematicMode = bCinematicMode && bHidePlayer;

	// If we have a pawn we need to determine if we should show/hide the player
	if (GetPawn() != NULL)
	{
		// Only hide the pawn if in cinematic mode and we want to
		if (bCinematicMode && bHidePawnInCinematicMode)
		{
			GetPawn()->SetActorHiddenInGame(true);
		}
		// Always safe to show the pawn when not in cinematic mode
		else if (!bCinematicMode)
		{
			GetPawn()->SetActorHiddenInGame(false);
		}
	}

	// Let the input system know about cinematic mode
	SetCinematicMode(bCinematicMode, bAffectsMovement, bAffectsTurning);

	// Replicate to the client
	ClientSetCinematicMode(bCinematicMode, bAffectsMovement, bAffectsTurning, bAffectsHUD);
}

/// @cond DOXYGEN_WARNINGS

void APlayerController::ClientSetCinematicMode_Implementation(bool bInCinematicMode, bool bAffectsMovement, bool bAffectsTurning, bool bAffectsHUD)
{
	bCinematicMode = bInCinematicMode;

	// If there's a HUD, set whether it should be shown or not
	if (MyHUD && bAffectsHUD)
	{
		MyHUD->bShowHUD = !bCinematicMode;
		ULocalPlayer* LocPlayer = Cast<ULocalPlayer>(Player);
		if (VirtualJoystick.IsValid())
		{
			VirtualJoystick->SetJoystickVisibility(MyHUD->bShowHUD, true);
		}
	}

	// Let the input system know about cinematic mode
	SetCinematicMode(bCinematicMode, bAffectsMovement, bAffectsTurning);
}

void APlayerController::ClientForceGarbageCollection_Implementation()
{
	GEngine->ForceGarbageCollection();
}

/// @endcond

void APlayerController::LevelStreamingStatusChanged(ULevelStreaming* LevelObject, bool bNewShouldBeLoaded, bool bNewShouldBeVisible, bool bNewShouldBlockOnLoad, int32 LODIndex)
{ 
	const bool bNewShouldBlockOnUnload = false;
	LevelStreamingStatusChanged(LevelObject, bNewShouldBeLoaded, bNewShouldBeVisible, bNewShouldBlockOnLoad, bNewShouldBlockOnUnload, LODIndex);
}

void APlayerController::LevelStreamingStatusChanged(ULevelStreaming* LevelObject, bool bNewShouldBeLoaded, bool bNewShouldBeVisible, bool bNewShouldBlockOnLoad, bool bNewShouldBlockOnUnload, int32 LODIndex)
{
	FNetLevelVisibilityTransactionId TransactionId;
	if (GetNetMode() == NM_Client)
	{
		TransactionId.SetIsClientInstigator(true);
	}
	else if (NetConnection)
	{
		// For server instigated visibility status changes we assign a transaction id that is used to ensure that we do not enable replication until visibility is confirmed by the client
		const FName PackageName = NetworkRemapPath(LevelObject->GetWorldAssetPackageFName(), true);

		TransactionId = NetConnection->UpdateLevelStreamStatusChangedTransactionId(LevelObject, PackageName, bNewShouldBeVisible);
	}

	ClientUpdateLevelStreamingStatus(NetworkRemapPath(LevelObject->GetWorldAssetPackageFName(), false), bNewShouldBeLoaded, bNewShouldBeVisible, bNewShouldBlockOnLoad, LODIndex, TransactionId, bNewShouldBlockOnUnload);
}

/// @cond DOXYGEN_WARNINGS

void APlayerController::ClientPrepareMapChange_Implementation(FName LevelName, bool bFirst, bool bLast)
{
	// Only call on the first local player controller to handle it being called on multiple PCs for splitscreen.
	if (GetGameInstance() == nullptr)
	{
		return;
	}

	APlayerController* PlayerController = GetGameInstance()->GetFirstLocalPlayerController();
	if( PlayerController != this )
	{
		return;
	}

	if (bFirst)
	{
		PendingMapChangeLevelNames.Empty();
		GetWorldTimerManager().ClearTimer(TimerHandle_DelayedPrepareMapChange);
	}
	PendingMapChangeLevelNames.Add(LevelName);
	if (bLast)
	{
		DelayedPrepareMapChange();
	}
}

/// @endcond

void APlayerController::DelayedPrepareMapChange()
{
	UWorld* World = GetWorld();
	if (World->IsPreparingMapChange())
	{
		// we must wait for the previous one to complete
		GetWorldTimerManager().SetTimer(TimerHandle_DelayedPrepareMapChange, this, &APlayerController::DelayedPrepareMapChange, 0.01f );
	}
	else
	{
		World->PrepareMapChange(PendingMapChangeLevelNames);
	}
}

/// @cond DOXYGEN_WARNINGS

void APlayerController::ClientCommitMapChange_Implementation()
{
	if (GetWorldTimerManager().IsTimerActive(TimerHandle_DelayedPrepareMapChange))
	{
		GetWorldTimerManager().SetTimer(TimerHandle_ClientCommitMapChange, this, &APlayerController::ClientCommitMapChange, 0.01f);
	}
	else
	{
		if (bAutoManageActiveCameraTarget)
		{
			if (GetPawnOrSpectator() != NULL)
			{
				AutoManageActiveCameraTarget(GetPawnOrSpectator());
			}
			else
			{
				AutoManageActiveCameraTarget(this);
			}
		}
		GetWorld()->CommitMapChange();
	}
}

void APlayerController::ClientCancelPendingMapChange_Implementation()
{
	GetWorld()->CancelPendingMapChange();
}

void APlayerController::ClientSetBlockOnAsyncLoading_Implementation()
{
	GetWorld()->bRequestedBlockOnAsyncLoading = true;
}

/// @endcond

void APlayerController::GetSeamlessTravelActorList(bool bToEntry, TArray<AActor*>& ActorList)
{
	if (MyHUD != NULL)
	{
		ActorList.Add(MyHUD);
	}

	// Should player camera persist or just be recreated?  (clients have to recreate on host)
	ActorList.Add(PlayerCameraManager);
}


void APlayerController::SeamlessTravelTo(APlayerController* NewPC)
{
	CleanUpAudioComponents();
}

void APlayerController::SeamlessTravelFrom(APlayerController* OldPC)
{
	// copy PlayerState data
	if (OldPC->PlayerState)
	{
		OldPC->PlayerState->Reset();
		OldPC->PlayerState->SeamlessTravelTo(PlayerState);

		//@fixme: need a way to replace PlayerStates that doesn't cause incorrect "player left the game"/"player entered the game" messages
		OldPC->PlayerState->Destroy();
		OldPC->SetPlayerState(NULL);
	}

	// Copy seamless travel state
	SeamlessTravelCount = OldPC->SeamlessTravelCount;
	LastCompletedSeamlessTravelCount = OldPC->LastCompletedSeamlessTravelCount;
}

void APlayerController::PostSeamlessTravel()
{
	// Track the last completed seamless travel for the player
	LastCompletedSeamlessTravelCount = SeamlessTravelCount;

	CleanUpAudioComponents();

	if (PlayerCameraManager == nullptr)
	{
		SpawnPlayerCameraManager();
	}

}

void APlayerController::OnAddedToPlayerControllerList()
{
	UWorld* World = GetWorld();
	// Possible we are moved into a world with no WorldPartitionSubsystem by the seamless travel (FSeamlessTravelHandler::StartTravel with no TransitionMap)
	if (UWorldPartitionSubsystem* WorldPartitionSubsystem = World->GetSubsystem<UWorldPartitionSubsystem>())
	{
		WorldPartitionSubsystem->RegisterStreamingSourceProvider(this);
	}
}

void APlayerController::OnRemovedFromPlayerControllerList()
{
	if (UWorld* World = GetWorld())
	{
		if (UWorldPartitionSubsystem* WorldPartitionSubsystem = World->GetSubsystem<UWorldPartitionSubsystem>())
		{
			verify(WorldPartitionSubsystem->UnregisterStreamingSourceProvider(this));
		}
	}
}

void APlayerController::GetStreamingSourceLocationAndRotation(FVector& OutLocation, FRotator& OutRotation) const
{
	if (!UE::Gameplay::CVars::ForceUsingCameraAsStreamingSource)
	{
		if (const AActor* ViewTarget = GetViewTarget())
		{
			ViewTarget->GetActorEyesViewPoint(OutLocation, OutRotation);
			return;
		}
	}

	GetPlayerViewPoint(OutLocation, OutRotation);
}

void APlayerController::GetStreamingSourceShapes(TArray<FStreamingSourceShape>& OutShapes) const
{
	if (StreamingSourceShapes.Num())
	{
		OutShapes.Append(StreamingSourceShapes);
	}
}

bool APlayerController::GetStreamingSource(FWorldPartitionStreamingSource& OutStreamingSource) const
{
	checkNoEntry();
	return false;
}

bool APlayerController::GetStreamingSources(TArray<FWorldPartitionStreamingSource>& OutStreamingSources) const
{
	const ENetMode NetMode = GetNetMode();
	const bool bIsServer = (NetMode == NM_DedicatedServer || NetMode == NM_ListenServer);
	if (IsStreamingSourceEnabled() && (IsLocalController() || bIsServer))
	{
		return GetStreamingSourcesInternal(OutStreamingSources);
	}
	return false;
}

bool APlayerController::GetStreamingSourcesInternal(TArray<FWorldPartitionStreamingSource>& OutStreamingSources) const
{
	FWorldPartitionStreamingSource& StreamingSource = OutStreamingSources.AddDefaulted_GetRef();
	GetStreamingSourceLocationAndRotation(StreamingSource.Location, StreamingSource.Rotation);
	StreamingSource.Name = GetFName();
	StreamingSource.TargetState = StreamingSourceShouldActivate() ? EStreamingSourceTargetState::Activated : EStreamingSourceTargetState::Loaded;
	StreamingSource.bBlockOnSlowLoading = StreamingSourceShouldBlockOnSlowStreaming();
	StreamingSource.DebugColor = StreamingSourceDebugColor;
	StreamingSource.Priority = GetStreamingSourcePriority();
	StreamingSource.bRemote = !IsLocalController();
	GetStreamingSourceShapes(StreamingSource.Shapes);
	return true;
}

/// @cond DOXYGEN_WARNINGS

void APlayerController::ClientEnableNetworkVoice_Implementation(bool bEnable)
{
	ToggleSpeaking(bEnable);
}

/// @endcond

void APlayerController::StartTalking()
{
	ToggleSpeaking(true);
}

void APlayerController::StopTalking()
{
	ToggleSpeaking(false);
}

void APlayerController::ToggleSpeaking(bool bSpeaking)
{
	ULocalPlayer* LP = Cast<ULocalPlayer>(Player);
	if (LP != NULL)
	{
		UWorld* World = GetWorld();
		if (bSpeaking)
		{
			UOnlineEngineInterface::Get()->StartNetworkedVoice(World, LP->GetControllerId());
		}
		else
		{
			UOnlineEngineInterface::Get()->StopNetworkedVoice(World, LP->GetControllerId());
		}
	}
}

/// @cond DOXYGEN_WARNINGS

void APlayerController::ClientVoiceHandshakeComplete_Implementation()
{
	MuteList.bHasVoiceHandshakeCompleted = true;
}

/// @endcond

void APlayerController::GameplayMutePlayer(const FUniqueNetIdRepl& PlayerNetId)
{
	if (PlayerNetId.IsValid())
	{
		MuteList.GameplayMutePlayer(this, PlayerNetId);
	}
}

void APlayerController::GameplayUnmutePlayer(const FUniqueNetIdRepl& PlayerNetId)
{
	if (PlayerNetId.IsValid())
	{
		MuteList.GameplayUnmutePlayer(this, PlayerNetId);
	}
}

void APlayerController::GameplayUnmuteAllPlayers()
{
	MuteList.GameplayUnmuteAllPlayers(this);
}

/// @cond DOXYGEN_WARNINGS

void APlayerController::ServerMutePlayer_Implementation(FUniqueNetIdRepl PlayerId)
{
	MuteList.ServerMutePlayer(this, PlayerId);
}

bool APlayerController::ServerMutePlayer_Validate(FUniqueNetIdRepl PlayerId)
{
	return PlayerId.IsValid();
}

void APlayerController::ServerUnmutePlayer_Implementation(FUniqueNetIdRepl PlayerId)
{
	MuteList.ServerUnmutePlayer(this, PlayerId);
}

bool APlayerController::ServerUnmutePlayer_Validate(FUniqueNetIdRepl PlayerId)
{
	return PlayerId.IsValid();
}

void APlayerController::ClientMutePlayer_Implementation(FUniqueNetIdRepl PlayerId)
{
	// Use the local player to determine the controller id
	ULocalPlayer* LP = Cast<ULocalPlayer>(Player);
	UWorld* World = GetWorld();

	// @todo: As of now we don't have a proper way to inform the client of the specific voice block reason
	// without changing the function signatures, therefore all server reasons are funneled into the client
	// as "muted" for the time being.
	MuteList.AddVoiceBlockReason(PlayerId.GetUniqueNetId(), EVoiceBlockReasons::Muted);

	if (LP != NULL && World)
	{
		// Have the voice subsystem mute this player
		UOnlineEngineInterface::Get()->MuteRemoteTalker(World, LP->GetControllerId(), PlayerId, false);
	}
}

void APlayerController::ClientUnmutePlayer_Implementation(FUniqueNetIdRepl PlayerId)
{
	// Use the local player to determine the controller id
	ULocalPlayer* LP = Cast<ULocalPlayer>(Player);
	UWorld* World = GetWorld();

	MuteList.RemoveVoiceBlockReason(PlayerId.GetUniqueNetId(), EVoiceBlockReasons::Muted);

	if (LP != NULL && World)
	{
		// Have the voice subsystem unmute this player
		UOnlineEngineInterface::Get()->UnmuteRemoteTalker(World, LP->GetControllerId(), PlayerId, false);
	}
}

void APlayerController::ClientUnmutePlayers_Implementation(const TArray<FUniqueNetIdRepl>& PlayerIds)
{
	ULocalPlayer* LP = Cast<ULocalPlayer>(Player);
	UWorld* World = GetWorld();

	// Use the local player to determine the controller id
	if (LP != NULL && World)
	{
		for (const FUniqueNetIdRepl& UnmuteId : PlayerIds)
		{
			if (UnmuteId.IsValid())
			{
				// Have the voice subsystem mute this player
				UOnlineEngineInterface::Get()->UnmuteRemoteTalker(World, LP->GetControllerId(), UnmuteId, false);
			}
		}
	}
}

void APlayerController::ServerBlockPlayer_Implementation(FUniqueNetIdRepl PlayerId)
{
	MuteList.ServerBlockPlayer(this, PlayerId);
}

bool APlayerController::ServerBlockPlayer_Validate(FUniqueNetIdRepl PlayerId)
{
	return PlayerId.IsValid() && PlayerState->GetUniqueId().IsValid();
}

void APlayerController::ServerUnblockPlayer_Implementation(FUniqueNetIdRepl PlayerId)
{
	MuteList.ServerUnblockPlayer(this, PlayerId);
}

bool APlayerController::ServerUnblockPlayer_Validate(FUniqueNetIdRepl PlayerId)
{
	return PlayerId.IsValid() && PlayerState->GetUniqueId().IsValid();
}

/// @endcond

APlayerController* APlayerController::GetPlayerControllerForMuting(const FUniqueNetIdRepl& PlayerNetId)
{
	return GetPlayerControllerFromNetId(GetWorld(), PlayerNetId);
}

bool APlayerController::IsPlayerMuted(const FUniqueNetId& PlayerId)
{
	return MuteList.IsPlayerMuted(PlayerId);
}

/// @cond DOXYGEN_WARNINGS

void APlayerController::ClientWasKicked_Implementation(const FText& KickReason)
{
}

void APlayerController::ClientStartOnlineSession_Implementation()
{
	if (IsPrimaryPlayer() && PlayerState && GetGameInstance() && GetGameInstance()->GetOnlineSession())
	{
		GetGameInstance()->GetOnlineSession()->StartOnlineSession(PlayerState->SessionName);
	}
}

void APlayerController::ClientEndOnlineSession_Implementation()
{
	if (IsPrimaryPlayer() && PlayerState && GetGameInstance() && GetGameInstance()->GetOnlineSession())
	{
		GetGameInstance()->GetOnlineSession()->EndOnlineSession(PlayerState->SessionName);
	}
}

/// @endcond

void APlayerController::ConsoleKey(FKey Key)
{
#if ALLOW_CONSOLE
	if (ULocalPlayer* LocalPlayer = Cast<ULocalPlayer>(Player))
	{
		if (LocalPlayer->ViewportClient && LocalPlayer->ViewportClient->ViewportConsole)
		{
			LocalPlayer->ViewportClient->ViewportConsole->InputKey(IPlatformInputDeviceMapper::Get().GetDefaultInputDevice(), Key, IE_Pressed);
		}
	}
#endif // ALLOW_CONSOLE
}
void APlayerController::SendToConsole(const FString& Command)
{
#if ALLOW_CONSOLE
	if (ULocalPlayer* LocalPlayer = Cast<ULocalPlayer>(Player))
	{
		if (LocalPlayer->ViewportClient && LocalPlayer->ViewportClient->ViewportConsole)
		{
			LocalPlayer->ViewportClient->ViewportConsole->ConsoleCommand(Command);
		}
	}
#endif // ALLOW_CONSOLE
}


bool APlayerController::IsPrimaryPlayer() const
{
	int32 SSIndex;
	return !IsSplitscreenPlayer(&SSIndex) || SSIndex == 0;
}

bool APlayerController::IsSplitscreenPlayer(int32* OutSplitscreenPlayerIndex) const
{
	bool bResult = false;

	if (OutSplitscreenPlayerIndex)
	{
		*OutSplitscreenPlayerIndex = NetPlayerIndex;
	}

	if (Player != NULL)
	{
		ULocalPlayer* const LP = Cast<ULocalPlayer>(Player);
		if (LP != NULL)
		{
			const TArray<ULocalPlayer*>& GamePlayers = LP->GetOuterUEngine()->GetGamePlayers(GetWorld());
			if (GamePlayers.Num() > 1)
			{
				if (OutSplitscreenPlayerIndex)
				{
					*OutSplitscreenPlayerIndex = GamePlayers.Find(LP);
				}
				bResult = true;
			}
		}
		else
		{
			UNetConnection* RemoteConnection = Cast<UNetConnection>(Player);
			if (RemoteConnection->Children.Num() > 0)
			{
				if (OutSplitscreenPlayerIndex)
				{
					*OutSplitscreenPlayerIndex = 0;
				}
				bResult = true;
			}
			else
			{
				UChildConnection* const ChildRemoteConnection = Cast<UChildConnection>(RemoteConnection);
				if (ChildRemoteConnection)
				{
					if (OutSplitscreenPlayerIndex && ChildRemoteConnection->Parent)
					{
						*OutSplitscreenPlayerIndex = ChildRemoteConnection->Parent->Children.Find(ChildRemoteConnection) + 1;
					}
					bResult = true;
				}
			}
		}
	}

	return bResult;
}


APlayerState* APlayerController::GetSplitscreenPlayerByIndex(int32 PlayerIndex) const
{
	APlayerState* Result = NULL;
	if ( Player != NULL )
	{
		if ( IsSplitscreenPlayer() )
		{
			ULocalPlayer* LP = Cast<ULocalPlayer>(Player);
			UNetConnection* RemoteConnection = Cast<UNetConnection>(Player);
			if ( LP != NULL )
			{
				const TArray<ULocalPlayer*>& GamePlayers = LP->GetOuterUEngine()->GetGamePlayers(GetWorld());
				// this PC is a local player
				if ( PlayerIndex >= 0 && PlayerIndex < GamePlayers.Num() )
				{
					ULocalPlayer* SplitPlayer = GamePlayers[PlayerIndex];
					Result = SplitPlayer->PlayerController->PlayerState;
				}
				else
				{
					UE_LOGF(LogPlayerController, Warning, "(%ls) APlayerController::%ls:GetSplitscreenPlayerByIndex: requested player at invalid index! PlayerIndex:%i NumLocalPlayers:%i",
						*GetFName().ToString(), *GetStateName().ToString(), PlayerIndex, GamePlayers.Num());
				}
			}
			else if ( RemoteConnection != NULL )
			{
				if ( GetNetMode() == NM_Client )
				{
					//THIS SHOULD NEVER HAPPEN - IF HAVE A REMOTECONNECTION, WE SHOULDN'T BE A CLIENT
					// this player is a client
					UE_LOGF(LogPlayerController, Warning, "(%ls) APlayerController::%ls:GetSplitscreenPlayerByIndex: CALLED ON CLIENT WITH VALID REMOTE NETCONNECTION!",
						*GetFName().ToString(), *GetStateName().ToString());
				}
				else
				{
					UChildConnection* ChildRemoteConnection = Cast<UChildConnection>(RemoteConnection);
					if ( ChildRemoteConnection != NULL )
					{
						// this player controller is not the primary player in the splitscreen layout
						UNetConnection* PrimaryConnection = ChildRemoteConnection->Parent;
						if ( PlayerIndex == 0 )
						{
							Result = PrimaryConnection->PlayerController->PlayerState;
						}
						else
						{
							PlayerIndex--;
							if ( PlayerIndex >= 0 && PlayerIndex < PrimaryConnection->Children.Num() )
							{
								ChildRemoteConnection = PrimaryConnection->Children[PlayerIndex];
								Result = ChildRemoteConnection->PlayerController->PlayerState;
							}
						}
					}
					else if ( RemoteConnection->Children.Num() > 0 )
					{
						// this PC is the primary splitscreen player
						if ( PlayerIndex == 0 )
						{
							// they want this player controller's PlayerState
							Result = PlayerState;
						}
						else
						{
							// our split-screen's PlayerState is being requested.
							PlayerIndex--;
							if ( PlayerIndex >= 0 && PlayerIndex < RemoteConnection->Children.Num() )
							{
								ChildRemoteConnection = RemoteConnection->Children[PlayerIndex];
								Result = ChildRemoteConnection->PlayerController->PlayerState;
							}
						}
					}
					else
					{
						UE_LOGF(LogPlayerController, Log, "(%ls) APlayerController::%ls:GetSplitscreenPlayerByIndex: %ls IS NOT THE PRIMARY CONNECTION AND HAS NO CHILD CONNECTIONS!", *GetName(), *GetStateName().ToString(), *Player->GetName());
					}
				}
			}
			else
			{
				UE_LOGF(LogPlayerController, Log, "(%ls) APlayerController::%ls:GetSplitscreenPlayerByIndex: %ls IS NOT A ULocalPlayer* AND NOT A RemoteConnection! (No valid UPlayer* reference)", *GetName(), *GetStateName().ToString(), *Player->GetName());
			}
		}
	}
	else
	{
		UE_LOGF(LogPlayerController, Log, "(%ls) APlayerController::%ls:GetSplitscreenPlayerByIndex: %ls", *GetName(), *GetStateName().ToString(), TEXT("NULL value for Player!"));
	}

	return Result;
}


int32 APlayerController::GetSplitscreenPlayerCount() const
{
	int32 Result = 0;

	if ( IsSplitscreenPlayer() )
	{
		if ( Player != NULL )
		{
			ULocalPlayer* const LP = Cast<ULocalPlayer>(Player);
			UNetConnection* RemoteConnection = Cast<UNetConnection>(Player);
			if ( LP != NULL )
			{
				Result = LP->GetOuterUEngine()->GetNumGamePlayers(GetWorld());
			}
			else if ( RemoteConnection != NULL )
			{
				if ( Cast<UChildConnection>(RemoteConnection) != NULL )
				{
					// we're the secondary (or otherwise) player in the split - we need to move up to the primary connection
					RemoteConnection = Cast<UChildConnection>(RemoteConnection)->Parent;
				}

				// add one for the primary player
				Result = RemoteConnection->Children.Num() + 1;
			}
			else
			{
				UE_LOGF(LogPlayerController, Log, "(%ls) APlayerController::%ls:GetSplitscreenPlayerCount NOT A ULocalPlayer* AND NOT A RemoteConnection!", *GetName(), *GetStateName().ToString());
			}
		}
		else
		{
			UE_LOGF(LogPlayerController, Log, "(%ls) APlayerController::%ls:GetSplitscreenPlayerCount called without a valid UPlayer* value!", *GetName(), *GetStateName().ToString());
		}
	}

	return Result;
}

/// @cond DOXYGEN_WARNINGS

void APlayerController::ClientSetForceMipLevelsToBeResident_Implementation( UMaterialInterface* Material, float ForceDuration, int32 CinematicTextureGroups )
{
	if ( Material != NULL && IsPrimaryPlayer() )
	{
		Material->SetForceMipLevelsToBeResident( false, false, ForceDuration, CinematicTextureGroups );
	}
}

void APlayerController::ClientPrestreamTextures_Implementation( AActor* ForcedActor, float ForceDuration, bool bEnableStreaming, int32 CinematicTextureGroups)
{
	if ( ForcedActor != NULL && IsPrimaryPlayer() )
	{
		ForcedActor->PrestreamTextures( ForceDuration, bEnableStreaming, CinematicTextureGroups );
	}
}

void APlayerController::ClientPlayForceFeedback_Internal_Implementation( UForceFeedbackEffect* ForceFeedbackEffect, FForceFeedbackParameters Params)
{
	if (ForceFeedbackEffect)
	{
		if (Params.Tag != NAME_None)
		{
			for (int32 Index = ActiveForceFeedbackEffects.Num() - 1; Index >= 0; --Index)
			{
				if (ActiveForceFeedbackEffects[Index].Parameters.Tag == Params.Tag)
				{
					// Reset the device properties on an active effect before removal
					ActiveForceFeedbackEffects[Index].ResetDeviceProperties();
					ActiveForceFeedbackEffects.RemoveAtSwap(Index);
				}
			}
		}
		
		ActiveForceFeedbackEffects.Emplace(ForceFeedbackEffect, Params, GetPlatformUserId());

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
		ForceFeedbackEffectHistoryEntries.Emplace(ActiveForceFeedbackEffects.Last(), GetWorld()->GetTimeSeconds());
#endif
	}
}

void APlayerController::K2_ClientPlayForceFeedback(class UForceFeedbackEffect* ForceFeedbackEffect, FName Tag, bool bLooping, bool bIgnoreTimeDilation, bool bPlayWhilePaused)
{
	FForceFeedbackParameters Params;
	Params.Tag = Tag;
	Params.bLooping = bLooping;
	Params.bIgnoreTimeDilation = bIgnoreTimeDilation;
	Params.bPlayWhilePaused = bPlayWhilePaused;
	ClientPlayForceFeedback(ForceFeedbackEffect, Params);
}

void APlayerController::ClientStopForceFeedback_Implementation( UForceFeedbackEffect* ForceFeedbackEffect, FName Tag)
{
	if (ForceFeedbackEffect == NULL && Tag == NAME_None)
	{
		// Reset all device properties
		for (FActiveForceFeedbackEffect& Effect : ActiveForceFeedbackEffects)
		{
			Effect.ResetDeviceProperties();
		}
		ActiveForceFeedbackEffects.Empty();
	}
	else
	{
		for (int32 Index = ActiveForceFeedbackEffects.Num() - 1; Index >= 0; --Index)
		{
			if (    (ForceFeedbackEffect == NULL || ActiveForceFeedbackEffects[Index].ForceFeedbackEffect == ForceFeedbackEffect)
				 && (Tag == NAME_None || ActiveForceFeedbackEffects[Index].Parameters.Tag == Tag) )
			{
				// Reset the device properties on an active effect before removal
				ActiveForceFeedbackEffects[Index].ResetDeviceProperties();
				ActiveForceFeedbackEffects.RemoveAtSwap(Index);
			}
		}
	}
}

/// @endcond

uint64 APlayerController::FDynamicForceFeedbackAction::HandleAllocator = 0;

bool APlayerController::FDynamicForceFeedbackAction::Update(const float DeltaTime, FForceFeedbackValues& Values)
{
	TimeElapsed += DeltaTime;

	if (TotalTime >= 0.f && TimeElapsed >= TotalTime)
	{
		return false;
	}

	ForceFeedbackDetails.Update(Values);
	return true;
}

/** Action that interpolates a component over time to a desired position */
class FLatentDynamicForceFeedbackAction : public FPendingLatentAction
{
public:
	/** Time over which interpolation should happen */
	float TotalTime;
	/** Time so far elapsed for the interpolation */
	float TimeElapsed;
	/** If we are currently running. If false, update will complete */
	uint8 bRunning:1;
	/** Whether the latent action is currently in the player controller's array */
	uint8 bAddedToPlayerController:1;

	TWeakObjectPtr<APlayerController> PlayerController;

	FDynamicForceFeedbackDetails ForceFeedbackDetails;

	/** Function to execute on completion */
	FName ExecutionFunction;
	/** Link to fire on completion */
	int32 OutputLink;
	/** Latent action ID */
	int32 LatentUUID;
	/** Object to call callback on upon completion */
	FWeakObjectPtr CallbackTarget;

	FLatentDynamicForceFeedbackAction(APlayerController* InPlayerController, const float InDuration, const FLatentActionInfo& LatentInfo)
		: TotalTime(InDuration)
		, TimeElapsed(0.f)
		, bRunning(true)
		, bAddedToPlayerController(false)
		, PlayerController(InPlayerController)
		, ExecutionFunction(LatentInfo.ExecutionFunction)
		, OutputLink(LatentInfo.Linkage)
		, LatentUUID(LatentInfo.UUID)
		, CallbackTarget(LatentInfo.CallbackTarget)
	{
	}

	~FLatentDynamicForceFeedbackAction()
	{
		if (bAddedToPlayerController)
		{
			if (APlayerController* PC = PlayerController.Get())
			{
				PC->LatentDynamicForceFeedbacks.Remove(LatentUUID);
			}
		}
	}

	virtual void UpdateOperation(FLatentResponse& Response) override
	{
		// Update elapsed time
		TimeElapsed += Response.ElapsedTime();

		const bool bComplete = (!bRunning || (TotalTime >= 0.f && TimeElapsed >= TotalTime) || !PlayerController.IsValid());

		if (APlayerController* PC = PlayerController.Get())
		{
			if (bComplete)
			{
				PC->LatentDynamicForceFeedbacks.Remove(LatentUUID);
				bAddedToPlayerController = false;
			}
			else
			{
				PC->LatentDynamicForceFeedbacks.Add(LatentUUID, &ForceFeedbackDetails);
				bAddedToPlayerController = true;
			}
		}

		Response.FinishAndTriggerIf(bComplete, ExecutionFunction, OutputLink, CallbackTarget);
	}

	virtual void NotifyObjectDestroyed() override
	{
		if (APlayerController* PC = PlayerController.Get())
		{
			PC->LatentDynamicForceFeedbacks.Remove(LatentUUID);
			bAddedToPlayerController = false;
		}
	}

	virtual void NotifyActionAborted() override
	{
		if (APlayerController* PC = PlayerController.Get())
		{
			PC->LatentDynamicForceFeedbacks.Remove(LatentUUID);
			bAddedToPlayerController = false;
		}
	}
};

void APlayerController::PlayDynamicForceFeedback(float Intensity, float Duration, bool bAffectsLeftLarge, bool bAffectsLeftSmall, bool bAffectsRightLarge, bool bAffectsRightSmall, TEnumAsByte<EDynamicForceFeedbackAction::Type> Action, FLatentActionInfo LatentInfo)
{
	FLatentActionManager& LatentActionManager = GetWorld()->GetLatentActionManager();
	FLatentDynamicForceFeedbackAction* LatentAction = LatentActionManager.FindExistingAction<FLatentDynamicForceFeedbackAction>(LatentInfo.CallbackTarget, LatentInfo.UUID);

	if (LatentAction)
	{
		if (Action == EDynamicForceFeedbackAction::Stop)
		{
			LatentAction->bRunning = false;
		}
		else
		{
			if (Action == EDynamicForceFeedbackAction::Start)
			{
				LatentAction->TotalTime = Duration;
				LatentAction->TimeElapsed = 0.f;
				LatentAction->bRunning = true;
			}

			LatentAction->ForceFeedbackDetails.Intensity = Intensity;
			LatentAction->ForceFeedbackDetails.bAffectsLeftLarge = bAffectsLeftLarge;
			LatentAction->ForceFeedbackDetails.bAffectsLeftSmall = bAffectsLeftSmall;
			LatentAction->ForceFeedbackDetails.bAffectsRightLarge = bAffectsRightLarge;
			LatentAction->ForceFeedbackDetails.bAffectsRightSmall = bAffectsRightSmall;
		}
	}
	else if (Action == EDynamicForceFeedbackAction::Start)
	{
		LatentAction = new FLatentDynamicForceFeedbackAction(this, Duration, LatentInfo);

		LatentAction->ForceFeedbackDetails.Intensity = Intensity;
		LatentAction->ForceFeedbackDetails.bAffectsLeftLarge = bAffectsLeftLarge;
		LatentAction->ForceFeedbackDetails.bAffectsLeftSmall = bAffectsLeftSmall;
		LatentAction->ForceFeedbackDetails.bAffectsRightLarge = bAffectsRightLarge;
		LatentAction->ForceFeedbackDetails.bAffectsRightSmall = bAffectsRightSmall;

		LatentActionManager.AddNewAction(LatentInfo.CallbackTarget, LatentInfo.UUID, LatentAction);
	}
}


void APlayerController::TestServerLevelVisibilityChange(const FName PackageName, const FName FileName)
{
#if !(UE_BUILD_TEST||UE_BUILD_SHIPPING)
	FUpdateLevelVisibilityLevelInfo LevelInfo;
	LevelInfo.bIsVisible = true;
	LevelInfo.PackageName = PackageName;
	LevelInfo.FileName = FileName;
	ServerUpdateLevelVisibility(LevelInfo);
#endif
}


FDynamicForceFeedbackHandle APlayerController::PlayDynamicForceFeedback(float Intensity, float Duration, bool bAffectsLeftLarge, bool bAffectsLeftSmall, bool bAffectsRightLarge, bool bAffectsRightSmall, EDynamicForceFeedbackAction::Type Action, FDynamicForceFeedbackHandle ActionHandle)
{
	FDynamicForceFeedbackHandle FeedbackHandle = 0;

	if (Action == EDynamicForceFeedbackAction::Stop)
	{
		if (ActionHandle > 0)
		{
			DynamicForceFeedbacks.Remove(ActionHandle);
		}
	}
	else
	{
		FDynamicForceFeedbackAction* FeedbackAction = (ActionHandle > 0 ? DynamicForceFeedbacks.Find(ActionHandle) : nullptr);

		if (FeedbackAction == nullptr && Action == EDynamicForceFeedbackAction::Start)
		{
			if (ActionHandle > 0)
			{
				if (ActionHandle <= FDynamicForceFeedbackAction::HandleAllocator)
				{
					// Restarting a stopped/finished index, this is fine
					FeedbackAction = &DynamicForceFeedbacks.Add(ActionHandle);
					FeedbackAction->Handle = ActionHandle;
				}
				else
				{
					UE_LOGF(LogPlayerController, Error, "Specifying an ID to start a dynamic force feedback with that has not yet been assigned is unsafe. No action has been started.");
				}
			}
			else
			{
				FeedbackAction = &DynamicForceFeedbacks.Add(++FDynamicForceFeedbackAction::HandleAllocator);
				FeedbackAction->Handle = FDynamicForceFeedbackAction::HandleAllocator;
			}
		}

		if (FeedbackAction)
		{
			if (Action == EDynamicForceFeedbackAction::Start)
			{
				FeedbackAction->TotalTime = Duration;
				FeedbackAction->TimeElapsed = 0.f;
			}

			FeedbackAction->ForceFeedbackDetails.Intensity = Intensity;
			FeedbackAction->ForceFeedbackDetails.bAffectsLeftLarge = bAffectsLeftLarge;
			FeedbackAction->ForceFeedbackDetails.bAffectsLeftSmall = bAffectsLeftSmall;
			FeedbackAction->ForceFeedbackDetails.bAffectsRightLarge = bAffectsRightLarge;
			FeedbackAction->ForceFeedbackDetails.bAffectsRightSmall = bAffectsRightSmall;

			FeedbackHandle = FeedbackAction->Handle;
		}
	}

	return FeedbackHandle;
}

void APlayerController::PlayHapticEffect(UHapticFeedbackEffect_Base* HapticEffect, EControllerHand Hand, float Scale, bool bLoop)
{
	if (HapticEffect)
	{
		switch (Hand)
		{
		case EControllerHand::Left:
			ActiveHapticEffect_Left.Reset();
			ActiveHapticEffect_Left = MakeShareable(new FActiveHapticFeedbackEffect(HapticEffect, Scale, bLoop));
			break;
		case EControllerHand::Right:
			ActiveHapticEffect_Right.Reset();
			ActiveHapticEffect_Right = MakeShareable(new FActiveHapticFeedbackEffect(HapticEffect, Scale, bLoop));
			break;
		case EControllerHand::Gun:
			ActiveHapticEffect_Gun.Reset();
			ActiveHapticEffect_Gun = MakeShareable(new FActiveHapticFeedbackEffect(HapticEffect, Scale, bLoop));
			break;
		case EControllerHand::HMD:
			ActiveHapticEffect_HMD.Reset();
			ActiveHapticEffect_HMD = MakeShareable(new FActiveHapticFeedbackEffect(HapticEffect, Scale, bLoop));
			break;
		default:
			UE_LOGF(LogPlayerController, Warning, "Invalid hand specified (%d) for haptic feedback effect %ls", (int32)Hand, *HapticEffect->GetName());
			break;
		}
	}
}

void APlayerController::StopHapticEffect(EControllerHand Hand)
{
	SetHapticsByValue(0.f, 0.f, Hand);
}

void APlayerController::SetDisableHaptics(bool bNewDisabled)
{
	if (bNewDisabled)
	{
		StopHapticEffect(EControllerHand::Left);
		StopHapticEffect(EControllerHand::Right);
		StopHapticEffect(EControllerHand::Gun);
	}

	bDisableHaptics = bNewDisabled;
}

static TAutoConsoleVariable<int32> CVarDisableHaptics(TEXT("input.DisableHaptics"),0,TEXT("If greater than zero, no haptic feedback is processed."));

void APlayerController::SetHapticsByValue(const float Frequency, const float Amplitude, EControllerHand Hand)
{
	bool bAreHapticsDisabled = bDisableHaptics || (CVarDisableHaptics.GetValueOnGameThread() > 0);
	if (bAreHapticsDisabled)
	{
		return;
	}

	if (Hand == EControllerHand::Left)
	{
		ActiveHapticEffect_Left.Reset();
	}
	else if (Hand == EControllerHand::Right)
	{
		ActiveHapticEffect_Right.Reset();
	}
	else if (Hand == EControllerHand::Gun)
	{
		ActiveHapticEffect_Gun.Reset();
	}
	else if (Hand == EControllerHand::HMD)
	{
		ActiveHapticEffect_HMD.Reset();
	}
	else
	{
		UE_LOGF(LogPlayerController, Warning, "Invalid hand specified (%d) for setting haptic feedback values (F: %f A: %f)", (int32)Hand, Frequency, Amplitude);
		return;
	}

	if (Player == nullptr)
	{
		return;
	}

	IInputInterface* InputInterface = FSlateApplication::Get().GetInputInterface();
	if (InputInterface)
	{
		const int32 ControllerId = CastChecked<ULocalPlayer>(Player)->GetControllerId();

		FHapticFeedbackValues Values(Frequency, Amplitude);
		InputInterface->SetHapticFeedbackValues(ControllerId, (int32)Hand, Values);
	}
}

void APlayerController::SetControllerLightColor(FColor Color)
{
	if (Player == nullptr)
	{
		return;
	}

	if (FSlateApplication::IsInitialized())
	{
		IInputInterface* InputInterface = FSlateApplication::Get().GetInputInterface();
		if (InputInterface)
		{
			const int32 ControllerId = CastChecked<ULocalPlayer>(Player)->GetControllerId();
			InputInterface->SetLightColor(ControllerId, Color);
		}
	}
}

void APlayerController::ResetControllerLightColor()
{
	if (Player == nullptr)
	{
		return;
	}

	if (FSlateApplication::IsInitialized())
	{
		IInputInterface* InputInterface = FSlateApplication::Get().GetInputInterface();
		if (InputInterface)
		{
			const int32 ControllerId = CastChecked<ULocalPlayer>(Player)->GetControllerId();
			InputInterface->ResetLightColor(ControllerId);
		}
	}
}

void APlayerController::SetControllerDeadZones(const float LeftDeadZone, const float RightDeadZone)
{
	if (Player == nullptr)
	{
		return;
	}

	if (FSlateApplication::IsInitialized())
	{
		IInputInterface* InputInterface = FSlateApplication::Get().GetInputInterface();
		if (InputInterface)
		{
			const int32 ControllerId = CastChecked<ULocalPlayer>(Player)->GetControllerId();
			FInputDeviceAnalogStickDeadZoneProperty DeadZoneLeft(EInputDeviceAnalogStickMask::Left, LeftDeadZone);
			InputInterface->SetDeviceProperty(ControllerId, &DeadZoneLeft);
			FInputDeviceAnalogStickDeadZoneProperty DeadZoneRight(EInputDeviceAnalogStickMask::Right, RightDeadZone);
			InputInterface->SetDeviceProperty(ControllerId, &DeadZoneRight);
		}
	}
}

void APlayerController::ResetControllerDeadZones()
{
	if (Player == nullptr)
	{
		return;
	}

	if (FSlateApplication::IsInitialized())
	{
		IInputInterface* InputInterface = FSlateApplication::Get().GetInputInterface();
		if (InputInterface)
		{
			const int32 ControllerId = CastChecked<ULocalPlayer>(Player)->GetControllerId();
			FInputDeviceAnalogStickDeadZoneProperty DeadZoneLeft(EInputDeviceAnalogStickMask::Left, 0.f);
			InputInterface->SetDeviceProperty(ControllerId, &DeadZoneLeft);
			FInputDeviceAnalogStickDeadZoneProperty DeadZoneRight(EInputDeviceAnalogStickMask::Right, 0.f);
			InputInterface->SetDeviceProperty(ControllerId, &DeadZoneRight);
		}
	}
}

void APlayerController::SetControllerTriggerReleaseThresholds(const float LeftThreshold, const float RightThreshold)
{
	if (Player == nullptr)
	{
		return;
	}

	if (FSlateApplication::IsInitialized())
	{
		IInputInterface* InputInterface = FSlateApplication::Get().GetInputInterface();
		if (InputInterface)
		{
			const int32 ControllerId = CastChecked<ULocalPlayer>(Player)->GetControllerId();
			FInputDeviceTriggerDynamicReleaseDeadZoneProperty ReleaseThresholdLeft(EInputDeviceTriggerMask::Left, LeftThreshold);
			InputInterface->SetDeviceProperty(ControllerId, &ReleaseThresholdLeft);
			FInputDeviceTriggerDynamicReleaseDeadZoneProperty ReleaseThresholdRight(EInputDeviceTriggerMask::Right, RightThreshold);
			InputInterface->SetDeviceProperty(ControllerId, &ReleaseThresholdRight);
		}
	}
}

void APlayerController::ResetControllerTriggerReleaseThresholds()
{
	if (Player == nullptr)
	{
		return;
	}

	if (FSlateApplication::IsInitialized())
	{
		IInputInterface* InputInterface = FSlateApplication::Get().GetInputInterface();
		if (InputInterface)
		{
			const int32 ControllerId = CastChecked<ULocalPlayer>(Player)->GetControllerId();
			FInputDeviceTriggerDynamicReleaseDeadZoneProperty ReleaseThreshold(EInputDeviceTriggerMask::All, 1.0f);
			InputInterface->SetDeviceProperty(ControllerId, &ReleaseThreshold);
		}
	}
}

void APlayerController::SetControllerGyroAutoCalibration(bool bEnabled)
{
	if (Player == nullptr)
	{
		return;
	}

	if (FSlateApplication::IsInitialized())
	{
		IInputInterface* InputInterface = FSlateApplication::Get().GetInputInterface();
		if (InputInterface)
		{
			const int32 ControllerId = CastChecked<ULocalPlayer>(Player)->GetControllerId();
			FInputDeviceGyroAutoCalibrationProperty GyroCalibration(bEnabled);
			InputInterface->SetDeviceProperty(ControllerId, &GyroCalibration);
		}
	}
}

void APlayerController::ProcessForceFeedbackAndHaptics(const float DeltaTime, const bool bGamePaused)
{
	if (Player == nullptr)
	{
		return;
	}

	ForceFeedbackValues.LeftLarge = ForceFeedbackValues.LeftSmall = ForceFeedbackValues.RightLarge = ForceFeedbackValues.RightSmall = 0.f;

	FHapticFeedbackValues LeftHaptics, RightHaptics, GunHaptics;
	bool bLeftHapticsNeedUpdate = false;
	bool bRightHapticsNeedUpdate = false;
	bool bGunHapticsNeedUpdate = false;
	FHapticFeedbackValues HMDHaptics;
	bool bHMDHapticsNeedUpdate = false;

	// Always process feedback by default, but if the game is paused then only static
	// effects that are flagged to play while paused will play
	bool bProcessFeedback = true;
#if WITH_EDITOR
	if (bProcessFeedback)
	{
		const ULocalPlayer* LocalPlayer = Cast<ULocalPlayer>(Player);
		if (LocalPlayer && LocalPlayer->ViewportClient)
		{
			if (FSceneViewport* Viewport = LocalPlayer->ViewportClient->GetGameViewport())
			{
				bProcessFeedback = !Viewport->GetPlayInEditorIsSimulate();
			}
		}
	}	
#endif

	UWorld* World = GetWorld();

	if (bProcessFeedback)
	{
		// --- Force Feedback --------------------------
		for (int32 Index = ActiveForceFeedbackEffects.Num() - 1; Index >= 0; --Index)
		{
			// If the game is paused, only tick force feedback effects that want to ignore time dilation
			if (!bGamePaused || ActiveForceFeedbackEffects[Index].Parameters.bPlayWhilePaused)
			{
				if (!ActiveForceFeedbackEffects[Index].Update(DeltaTime, ForceFeedbackValues))
				{
					// Reset any device properties that may need it (i.e. trigger resistance) 
					ActiveForceFeedbackEffects[Index].ResetDeviceProperties();
					ActiveForceFeedbackEffects.RemoveAtSwap(Index);
				}
			}
		}

		const bool bProcessDynamicFeedback = !bGamePaused;
		if (bProcessDynamicFeedback)
		{
			for (TSortedMap<uint64, FDynamicForceFeedbackAction>::TIterator It(DynamicForceFeedbacks.CreateIterator()); It; ++It)
			{
				if (!It.Value().Update(DeltaTime, ForceFeedbackValues))
				{
					It.RemoveCurrent();
				}
			}

			for (const TPair<int32, FDynamicForceFeedbackDetails*>& DynamicEntry : LatentDynamicForceFeedbacks)
			{
				DynamicEntry.Value->Update(ForceFeedbackValues);
			}
		}

		if (FForceFeedbackManager* ForceFeedbackManager = FForceFeedbackManager::Get(World))
		{
			ForceFeedbackManager->Update(GetFocalLocation(), ForceFeedbackValues, GetPlatformUserId());
		}

		// Apply ForceFeedbackScale
		ForceFeedbackValues.LeftLarge  = FMath::Clamp(ForceFeedbackValues.LeftLarge * ForceFeedbackScale, 0.f, 1.f);
		ForceFeedbackValues.RightLarge = FMath::Clamp(ForceFeedbackValues.RightLarge * ForceFeedbackScale, 0.f, 1.f);
		ForceFeedbackValues.LeftSmall  = FMath::Clamp(ForceFeedbackValues.LeftSmall * ForceFeedbackScale, 0.f, 1.f);
		ForceFeedbackValues.RightSmall = FMath::Clamp(ForceFeedbackValues.RightSmall * ForceFeedbackScale, 0.f, 1.f);

		// --- Haptic Feedback -------------------------
		if (bProcessDynamicFeedback)
		{
			if (ActiveHapticEffect_Left.IsValid())
			{
				const bool bPlaying = ActiveHapticEffect_Left->Update(DeltaTime, LeftHaptics);
				if (!bPlaying)
				{
					ActiveHapticEffect_Left->bLoop ? ActiveHapticEffect_Left->Restart() : ActiveHapticEffect_Left.Reset();
				}

				bLeftHapticsNeedUpdate = true;
			}

			if (ActiveHapticEffect_Right.IsValid())
			{
				const bool bPlaying = ActiveHapticEffect_Right->Update(DeltaTime, RightHaptics);
				if (!bPlaying)
				{
					ActiveHapticEffect_Right->bLoop ? ActiveHapticEffect_Right->Restart() : ActiveHapticEffect_Right.Reset();
				}

				bRightHapticsNeedUpdate = true;
			}

			if (ActiveHapticEffect_Gun.IsValid())
			{
				const bool bPlaying = ActiveHapticEffect_Gun->Update(DeltaTime, GunHaptics);
				if (!bPlaying)
				{
					ActiveHapticEffect_Gun->bLoop ? ActiveHapticEffect_Gun->Restart() : ActiveHapticEffect_Gun.Reset();
				}

				bGunHapticsNeedUpdate = true;
			}
			if (ActiveHapticEffect_HMD.IsValid())
			{
				const bool bPlaying = ActiveHapticEffect_HMD->Update(DeltaTime, HMDHaptics);
				if (!bPlaying)
				{
					ActiveHapticEffect_HMD->bLoop ? ActiveHapticEffect_HMD->Restart() : ActiveHapticEffect_HMD.Reset();
				}

				bHMDHapticsNeedUpdate = true;
			}
		}
	}

	if (FSlateApplication::IsInitialized())
	{
		int32 ControllerId = GetInputIndex();

		if (ControllerId != INVALID_CONTROLLERID)
		{
			IInputInterface* InputInterface = FSlateApplication::Get().GetInputInterface();
			if (InputInterface)
			{
				// Adjust the ControllerId to account for the controller ID offset applied in UGameViewportClient::InputKey/Axis
				// to play the force feedback on the correct controller if the offset player gamepad IDs feature is in use. 
				const int32 NumLocalPlayers = World->GetGameInstance()->GetNumLocalPlayers();
				if (NumLocalPlayers > 1 && GetDefault<UGameMapsSettings>()->bOffsetPlayerGamepadIds)
				{
					--ControllerId;
				}

				UpdateForceFeedback(InputInterface, ControllerId);

				const bool bAreHapticsDisabled = (CVarDisableHaptics.GetValueOnGameThread() > 0) || bDisableHaptics;
				if (!bAreHapticsDisabled)
				{
					// Haptic Updates
					if (bLeftHapticsNeedUpdate)
					{
						InputInterface->SetHapticFeedbackValues(ControllerId, (int32)EControllerHand::Left, LeftHaptics);
					}
					if (bRightHapticsNeedUpdate)
					{
						InputInterface->SetHapticFeedbackValues(ControllerId, (int32)EControllerHand::Right, RightHaptics);
					}
					if (bGunHapticsNeedUpdate)
					{
						InputInterface->SetHapticFeedbackValues(ControllerId, (int32)EControllerHand::Gun, GunHaptics);
					}
					if (bHMDHapticsNeedUpdate)
					{
						InputInterface->SetHapticFeedbackValues(ControllerId, (int32)EControllerHand::HMD, HMDHaptics);
					}
				}
			}
		}
	}
}

void APlayerController::UpdateForceFeedback(IInputInterface* InputInterface, const int32 ControllerId)
{
	InputInterface->SetForceFeedbackChannelValues(ControllerId, (bForceFeedbackEnabled ? ForceFeedbackValues : FForceFeedbackValues()));
}

/// @cond DOXYGEN_WARNINGS

void APlayerController::ClientStartCameraShake_Implementation( TSubclassOf<class UCameraShakeBase> Shake, float Scale, ECameraShakePlaySpace PlaySpace, FRotator UserPlaySpaceRot )
{
	if (PlayerCameraManager != NULL)
	{
		PlayerCameraManager->StartCameraShake(Shake, Scale, PlaySpace, UserPlaySpaceRot);
	}
}

void APlayerController::ClientStopCameraShake_Implementation( TSubclassOf<class UCameraShakeBase> Shake, bool bImmediately )
{
	if (PlayerCameraManager != NULL)
	{
		PlayerCameraManager->StopAllInstancesOfCameraShake(Shake, bImmediately);
	}
}

void APlayerController::ClientStartCameraShakeFromSource(TSubclassOf<class UCameraShakeBase> Shake, class UCameraShakeSourceComponent* SourceComponent)
{
	if (PlayerCameraManager != NULL)
	{
		PlayerCameraManager->StartCameraShakeFromSource(Shake, SourceComponent);
	}
}

void APlayerController::ClientStopCameraShakesFromSource(class UCameraShakeSourceComponent* SourceComponent, bool bImmediately)
{
	if (PlayerCameraManager != NULL)
	{
		PlayerCameraManager->StopAllCameraShakesFromSource(SourceComponent, bImmediately);
	}
}

void APlayerController::ClientSpawnGenericCameraLensEffect_Implementation(TSubclassOf<class AActor> LensEffectEmitterClass)
{
	if (PlayerCameraManager != NULL)
	{
		PlayerCameraManager->AddGenericCameraLensEffect(*LensEffectEmitterClass);
	}
}

void APlayerController::ClientSpawnCameraLensEffect_Implementation( TSubclassOf<AEmitterCameraLensEffectBase> LensEffectEmitterClass )
{
	if (PlayerCameraManager != NULL)
	{
		PlayerCameraManager->AddGenericCameraLensEffect(*LensEffectEmitterClass);
	}
}

void APlayerController::ClientClearCameraLensEffects_Implementation()
{
	if (PlayerCameraManager != NULL)
	{
		PlayerCameraManager->ClearCameraLensEffects();
	}
}

/// @endcond

void APlayerController::ReceivedGameModeClass(TSubclassOf<AGameModeBase> GameModeClass)
{
}

void APlayerController::ReceivedSpectatorClass(TSubclassOf<ASpectatorPawn> SpectatorClass)
{
	if (IsInState(NAME_Spectating))
	{
		if (GetSpectatorPawn() == NULL)
		{
			BeginSpectatingState();
		}
	}
}

void APlayerController::SetPawn(APawn* InPawn)
{
	if (GetLocalRole() == ROLE_Authority && UE::Net::FReplicationSystemUtil::GetNetHandle(this).IsValid())
	{
		if (const UReplicationSystem* ReplicationSystem = UE::Net::FReplicationSystemUtil::GetReplicationSystem(this))
		{
			if (APawn* PrevPawn = GetPawn(); PrevPawn != InPawn)
			{
				if (IsValid(PrevPawn))
				{
					UE::Net::FReplicationSystemUtil::RemoveDependentActor(this, PrevPawn);
				}

				if (IsValid(InPawn))
				{
					UE::Net::FReplicationSystemUtil::AddDependentActor(this, InPawn, UE::Net::EDependentObjectSchedulingHint::ScheduleBeforeParent);
				}
			}
		}
	}

	if (InPawn == NULL)
	{
		// Attempt to move the PC to the current camera location if no pawn was specified
		const FVector NewLocation = (PlayerCameraManager != NULL) ? PlayerCameraManager->GetCameraLocation() : GetSpawnLocation();
		SetSpawnLocation(NewLocation);

		if (bAutoManageActiveCameraTarget)
		{
			AutoManageActiveCameraTarget(this);
		}
	}

	Super::SetPawn(InPawn);

	// If we have a pawn we need to determine if we should show/hide the player for cinematic mode
	if (GetPawn() && bCinematicMode && bHidePawnInCinematicMode)
	{
		GetPawn()->SetActorHiddenInGame(true);
	}
}

void APlayerController::GetLifetimeReplicatedProps(TArray< FLifetimeProperty > & OutLifetimeProps) const
{
	Super::GetLifetimeReplicatedProps(OutLifetimeProps);
	
	FDoRepLifetimeParams Params;
	Params.bIsPushBased = UE::Gameplay::CVars::bIsPlayerControllerPushBased;
	Params.Condition = COND_OwnerOnly;
	// These used to only replicate if PlayerCameraManager->GetViewTargetPawn() != GetPawn()
	// But, since they also don't update unless that condition is true, these values won't change, thus won't send
	// This is a little less efficient, but fits into the new condition system well, and shouldn't really add much overhead
	DOREPLIFETIME_WITH_PARAMS_FAST(APlayerController, TargetViewRotation, Params);

	// Replicate SpawnLocation for remote spectators
	DOREPLIFETIME_WITH_PARAMS_FAST(APlayerController, SpawnLocation, Params);
}

#if UE_WITH_REMOTE_OBJECT_HANDLE
static void UpdateNetConnectionControllerPosition(UNetConnection* NetConnection)
{
	AActor* ControllerActor = Cast<AActor>(NetConnection->PlayerController);
	bool bNoPawnPlayerControllerPositionSet = false;
	if (ControllerActor)
	{
		if (NetConnection->ViewTarget.GetResidence() == EResidence::Remote)
		{
			UE_LOGF(LogPlayerController, Warning, "UpdateNetConnectionControllerPosition: Incoming ViewTarget is remote.");
		}
		else
		{
			if (APlayerController* PlayerController = Cast<APlayerController>(NetConnection->ViewTarget))
			{
				FVector TrueViewLocation;
				FRotator ViewRotation;
				PlayerController->GetPlayerViewPoint(TrueViewLocation, ViewRotation);
				ControllerActor->SetActorLocation(TrueViewLocation);
				UE_LOGF(LogPlayerController, Verbose, "[%ls] UpdateNetConnectionControllerPosition: Setting NoPawnPlayerController %ls position to %ls using position from PlayerController %ls",
					*FRemoteServerId::GetLocalServerId().ToString(),
					*ControllerActor->GetName(), *(TrueViewLocation.ToString()), *PlayerController->GetName());
				bNoPawnPlayerControllerPositionSet = true;
			}
			else if (AActor* TargetActor = Cast<AActor>(NetConnection->ViewTarget))
			{
				FVector TrueViewLocation = TargetActor->GetActorLocation();
				ControllerActor->SetActorLocation(TrueViewLocation);
				UE_LOGF(LogPlayerController, Verbose, "[%ls] UpdateNetConnectionControllerPosition: Setting NoPawnPlayerController %ls position to %ls using position from PlayerCharacter %ls",
					*FRemoteServerId::GetLocalServerId().ToString(),
					*ControllerActor->GetName(), *(TrueViewLocation.ToString()), *TargetActor->GetName());
				bNoPawnPlayerControllerPositionSet = true;
			}
		}
	}
	if (!bNoPawnPlayerControllerPositionSet)
	{
		UE_LOGF(LogPlayerController, Warning, "[%ls] UpdateNetConnectionControllerPosition: Failed to set position of NoPawnPlayerController %ls",
			*FRemoteServerId::GetLocalServerId().ToString(),
			*GetNameSafe(NetConnection->PlayerController));
	}
}

void APlayerController::PostMigrate(const FUObjectMigrationContext& MigrationContext)
{
	auto RemoveFromReplicationSystem = [](UNetDriver* NetDriver, AActor* Actor)
	{
		if (NetDriver->GetReplicationSystem())
		{
			FStopReplicatingActorParams StopRepParams(EEndPlayReason::Destroyed);
			StopRepParams.StopReplicatingReason = UE::Net::EStopReplicatingReason::ProxyReuse;
			UE::Net::FDeferredReplicationSystemCalls::StopReplicatingActorDefer(Actor, StopRepParams);
		}
		else
		{
			NetDriver->RemoveNetworkActor(Actor);
		}
	};

	auto AddToReplicationSystem = [](UNetDriver* NetDriver, AActor* Actor)
	{
		if (NetDriver->GetReplicationSystem())
		{
			UE::Net::FDeferredReplicationSystemCalls::StartReplicatingActorDefer(Actor);
		}
		else
		{
			NetDriver->AddNetworkActor(Actor);
		}
	};

	// The ANoPawnPlayerController is server specific and should never be migrated between servers.
	ensure(!IsA(ANoPawnPlayerController::StaticClass()));

	// APlayerController::NetConnection is set to null when serialized between servers so it should be null here. This function
	// is expected to set a new value of NetConnection if and when required.
	ensure(NetConnection == nullptr);

	// Make sure we have the prerequisites
	UWorld* World = GetWorld();
	UNetDriver* NetDriver = World ? World->GetNetDriver() : nullptr;
	if (!ensureMsgf(NetDriver, TEXT("%s::%hs: World or NetDriver not set."), *GetName(), __FUNCTION__))
	{
		return;
	}

	// Update the NetConnection on the player controller because it will be set to null when serialized. It's important that the NetConnection variable
	// is always updated, even when borrowed, because there is code that depends on that value and won't work if it's null.
	// We need to do this before Super::PostMigrate because RPCs will not Queue without Player set.
	if (MigrationContext.MigrationSide == EObjectMigrationSide::Receive)
	{
		// Set NetConnection to the local connection that matches the player id in the migrated player controller.
		NetConnection = NetDriver->FindConnection(CachedConnectionPlayerId);
			
		if (!ensureMsgf(NetConnection, TEXT("APlayerController::PostMigrate: Failed to find a NetConnection associated with player id %s"), *CachedConnectionPlayerId.ToDebugString()))
		{
			return;
		}

		// Set Player at the same time as NetConnection because it's used in APlayerController::GetConnection().
		Player = NetConnection;
	}

	Super::PostMigrate(MigrationContext);
		
	const bool bHasOwnership = UE::RemoteObject::Handle::IsOwned(this);
	
	if (MigrationContext.MigrationSide == EObjectMigrationSide::Receive)
	{
		EObjectMigrationRecvType MigrationRecvType = MigrationContext.GetObjectMigrationRecvType(this);

		UE_LOGF(LogPlayerController, Verbose, "[%ls] APlayerController::PostMigrate [%ls]: %ls bHasOwnership=%d MigrationSide=Receive RecvType=%ls PlayerId=%ls", 
			   *FRemoteServerId::GetLocalServerId().ToString(), *FRemoteObjectId(this).ToString(), *GetName(), bHasOwnership, ToString(MigrationRecvType), *CachedConnectionPlayerId.ToDebugString());

		const bool bAttachControllerToConnection = (MigrationRecvType == EObjectMigrationRecvType::AssignedOwnership);
		if (bAttachControllerToConnection)
		{	
			// It's assumed the current connection for the player id should be using an ANoPawnPlayerController.
			ensure(NetConnection->PlayerController);
			ensure(NetConnection->PlayerController.IsA<ANoPawnPlayerController>());
			
			ANoPawnPlayerController* PrevNoPawnPlayerController = Cast<ANoPawnPlayerController>(NetConnection->PlayerController);
			if (PrevNoPawnPlayerController)
			{
				// Detach the network connection and its player controller.
				NetConnection->PlayerController = nullptr;
				NetConnection->OwningActor = nullptr;
				PrevNoPawnPlayerController->NetConnection = nullptr;
				PrevNoPawnPlayerController->Player = nullptr;

				// Update the cached ANoPawnPlayerController to account for cases where this connection was established with an
				// instance of ANoPawnPlayerController outside of APlayerController::PostMigrate() (e.g. a connection was opened
				// with NMT_JoinNoPawn or NMT_JoinNoPawnSplit). Without this code we might lose reference to an instance of 
				// ANoPawnPlayerController and create a controller leak.
				NetConnection->CachedNoPawnPlayerController = PrevNoPawnPlayerController;

				// Attach this migrated player controller to the network connection.
				NetConnection->PlayerController = this;
				NetConnection->OwningActor = this;

				// Update the values of this migrated player controller that are associated with the connection.
				NetPlayerIndex = PrevNoPawnPlayerController->NetPlayerIndex;
				SetClientHandshakeId(NetConnection->GetClientHandshakeId());
				LocalPlayerConnectionIdentifier = PrevNoPawnPlayerController->LocalPlayerConnectionIdentifier;
				CachedConnectionPlayerId = PrevNoPawnPlayerController->CachedConnectionPlayerId;

				// Stop replicating the old ANoPawnPlayerController since it's no longer used.
				RemoveFromReplicationSystem(NetDriver, PrevNoPawnPlayerController);

				UpdateOwningNetConnection();

				UE_LOGF(LogPlayerController, Verbose, "[%ls] APlayerController::PostMigrate [%ls]: Binding %ls to connection %ls",
					   *FRemoteServerId::GetLocalServerId().ToString(), *FRemoteObjectId(this).ToString(),
					   *GetName(), *NetConnection->GetName());

				// There should only be one registered controller in the world for this connection and associated player.
				World->RemoveController(PrevNoPawnPlayerController);
				World->AddController(this);
			}
		}
	}
	else if (MigrationContext.MigrationSide == EObjectMigrationSide::Send)
	{
		EObjectMigrationSendType MigrationSendType = MigrationContext.GetObjectMigrationSendType(this);

		UE_LOGF(LogPlayerController, Verbose, "[%ls] APlayerController::PostMigrate [%ls]: %ls bHasOwnership=%d MigrationSide=Send SendType=%ls PlayerId=%ls",
			   *FRemoteServerId::GetLocalServerId().ToString(), *FRemoteObjectId(this).ToString(), *GetName(), bHasOwnership, ToString(MigrationSendType), *CachedConnectionPlayerId.ToDebugString());

		const bool bDetachControllerToConnection = (MigrationSendType == EObjectMigrationSendType::ReassignOwnership);

		if (bDetachControllerToConnection)
		{
			// Lookup the local connection that matches the player id in the player controller being migrated away.
			UNetConnection* PrevNetConnection = NetDriver->FindConnection(CachedConnectionPlayerId);
		
			if (!ensureMsgf(PrevNetConnection, TEXT("APlayerController::PostMigrate: Failed to find a connection associated with player id %s"), *CachedConnectionPlayerId.ToDebugString()))
			{
				return;
			}

			if (!ensureMsgf(PrevNetConnection->PlayerController == this, TEXT("APlayerController::PostMigrate: Connection doesn't match player controller being migrated away.")))
			{
				return;
			}

			if (ensure(PrevNetConnection->PlayerController))
			{
				// It's assumed this connection has a player controller associated that is not an instance of ANoPawnPlayerController.
				ensure(!PrevNetConnection->PlayerController.IsA<ANoPawnPlayerController>());
				ensure(PrevNetConnection->PlayerController != PrevNetConnection->CachedNoPawnPlayerController);

				// Detach the network connection and its player controller.
				PrevNetConnection->PlayerController->NetConnection = nullptr;
				PrevNetConnection->PlayerController->Player = nullptr;
				PrevNetConnection->PlayerController = nullptr;
				PrevNetConnection->OwningActor = nullptr;

				auto BindAndInitializeConnectionAndController = [](UNetConnection* Connection, ANoPawnPlayerController* NoPawnPlayerController, const APlayerController* OldPlayerController)
				{
					// Bind the connection with the ANoPawnPlayerController.
					Connection->PlayerController = NoPawnPlayerController;
					Connection->OwningActor = NoPawnPlayerController;
					NoPawnPlayerController->NetConnection = Connection;
					NoPawnPlayerController->Player = Connection;

					// Make sure that the ANoPawnPlayerController has the correct connection related state.
					NoPawnPlayerController->CachedConnectionPlayerId = OldPlayerController->CachedConnectionPlayerId;
					NoPawnPlayerController->NetPlayerIndex = OldPlayerController->NetPlayerIndex;
					NoPawnPlayerController->SetClientHandshakeId(Connection->GetClientHandshakeId());
					NoPawnPlayerController->LocalPlayerConnectionIdentifier = OldPlayerController->LocalPlayerConnectionIdentifier;
				};

				// If the cached no pawn player controller was created in a different world, destroy it so that it will be re-created. This
				// can happen during seamless travel and the world is destroyed but the connection remains open.
				if (PrevNetConnection->CachedNoPawnPlayerController)
				{
					if (PrevNetConnection->CachedNoPawnPlayerController->GetWorld() != World)
					{
						UE_LOGF(LogPlayerController, Verbose, "APlayerController::PostMigrate(): Destroying cached instance ANoPawnPlayerController because it was spawned in a different world.");

						PrevNetConnection->CachedNoPawnPlayerController->Destroy();
						PrevNetConnection->CachedNoPawnPlayerController = nullptr;
					}
				}

				// If there is no cached instances of ANoPawnPlayerController, make sure one is spawned.
				if (PrevNetConnection->CachedNoPawnPlayerController == nullptr)
				{
					UE_LOGF(LogPlayerController, Verbose, "APlayerController::PostMigrate(): Spawning a new instance of ANoPawnPlayerController for connection %ls", *PrevNetConnection->GetName());

					FActorSpawnParameters SpawnInfo;
					SpawnInfo.ObjectFlags |= RF_Transient;
					SpawnInfo.bDeferConstruction = true;
					ANoPawnPlayerController* NewNoPawnPlayerController = World->SpawnActor<ANoPawnPlayerController>(ANoPawnPlayerController::StaticClass(), FVector(), FRotator(), SpawnInfo);
					if (ensure(NewNoPawnPlayerController))
					{
						// Attach this new instance of ANoPawnPlayerController to the network connection.
						BindAndInitializeConnectionAndController(PrevNetConnection, NewNoPawnPlayerController, this);

						// Initial configuration for the ANoPawnPlayerController.
						NewNoPawnPlayerController->SetRole(ROLE_Authority);
						NewNoPawnPlayerController->SetReplicates(true);
						NewNoPawnPlayerController->SetAutonomousProxy(true);

						// Finalizing spawning of the controller.
						// When using Iris this will call AActor::UpdateOwningNetConnection() and StartReplication() (immediately and not deferred).
						NewNoPawnPlayerController->FinishSpawning(FTransform());

						// Cache the instances of ANoPawnPlayerController to avoid respawning it on player migration.
						PrevNetConnection->CachedNoPawnPlayerController = NewNoPawnPlayerController;
					}
				}
				else
				{
					ensure(PrevNetConnection->CachedNoPawnPlayerController->IsA<ANoPawnPlayerController>());
				
					ANoPawnPlayerController* CachedNoPawnPlayerController = PrevNetConnection->CachedNoPawnPlayerController;

					// Attach this cached instance of ANoPawnPlayerController to the network connection.
					BindAndInitializeConnectionAndController(PrevNetConnection, CachedNoPawnPlayerController, this);
				
					// Start replicating the ANoPawnPlayerController.
					AddToReplicationSystem(NetDriver, CachedNoPawnPlayerController);

					// Since the player controller is being re-used, ensure that its initial state is the same as if it were just spawned.
					//
					// NOTE: This currently commented out because PostMigrate() can be called while Iris is replicating and Iris will ensure
					//		 when calling AActor::UpdateOwningNetConnection() during replication. This seems to be okay for now since
					//		 none of the state from ANoPawnPlayerController is changing between migrations.
					//CachedNoPawnPlayerController->SetRole(ROLE_Authority);
					//CachedNoPawnPlayerController->SetReplicates(true); // Calls AActor::UpdateOwningNetConnection().
					//CachedNoPawnPlayerController->SetAutonomousProxy(true);
				}

				// Update the position of the ANoPawnPlayerController during migration, so it will be correct when used for relevancy check. 
				UpdateNetConnectionControllerPosition(PrevNetConnection);

				UE_LOGF(LogPlayerController, Verbose, "[%ls] APlayerController::PostMigrate [%ls]: Replacing %ls with new player controller %ls on connection %ls",
					   *FRemoteServerId::GetLocalServerId().ToString(), *FRemoteObjectId(this).ToString(),
					   *GetName(), *PrevNetConnection->PlayerController->GetName(), *PrevNetConnection->GetName());

				ensure(PrevNetConnection->PlayerController);
				ensure(PrevNetConnection->OwningActor);

				// There should only be one registered controller in the world for this connection and associated player.
				World->RemoveController(this);
				World->AddController(PrevNetConnection->PlayerController);
			}
		}
	}
}
#endif

void APlayerController::Serialize(FArchive& Ar)
{
#if UE_WITH_REMOTE_OBJECT_HANDLE
	if (Ar.IsMigratingRemoteObjects())
	{
		if (Ar.IsLoading() || Ar.IsSaving())
		{
			if (NetConnection)
			{
				CachedConnectionPlayerId = NetConnection->PlayerId;
			}

			Ar << CachedConnectionPlayerId;

			// The these properties must be nulled out here because they are server specific. This must happen here
			// so they are serialized as null in Super::Serialize().
			NetConnection = nullptr;
			Player = nullptr;
		}
	}
		
	Super::Serialize(Ar);
#else
	Super::Serialize(Ar);
#endif
}

void APlayerController::SetTargetViewRotation(const FRotator& InRotation)
{
	if (UE::Gameplay::CVars::bIsPlayerControllerPushBased)
	{
		COMPARE_ASSIGN_AND_MARK_PROPERTY_DIRTY(APlayerController, TargetViewRotation, InRotation, this);
		return;
	}

	TargetViewRotation = InRotation;
}

void APlayerController::SetPlayer( UPlayer* InPlayer )
{
	FMoviePlayerProxyBlock MoviePlayerBlock;
	check(InPlayer!=NULL);

	const bool bIsSameLevel = InPlayer->PlayerController && (InPlayer->PlayerController->GetLevel() == GetLevel());
	// Detach old player if it's in the same level.
	if (bIsSameLevel)
	{
		InPlayer->PlayerController->Player = NULL;
	}

	// Set the viewport.
	Player = InPlayer;
	InPlayer->PlayerController = this;

	// cap outgoing rate to max set by server
	UNetDriver* Driver = GetWorld()->GetNetDriver();
	if( (ClientCap>=2600) && Driver && Driver->ServerConnection )
	{
		Player->CurrentNetSpeed = Driver->ServerConnection->CurrentNetSpeed = FMath::Clamp( ClientCap, 1800, Driver->MaxClientRate );
	}

	// initializations only for local players
	ULocalPlayer *LP = Cast<ULocalPlayer>(InPlayer);
	if (LP != NULL)
	{
		// Clients need this marked as local (server already knew at construction time)
		SetAsLocalPlayerController();
		LP->InitOnlineSession();
		InitInputSystem();
	}
	else
	{
		NetConnection = Cast<UNetConnection>(InPlayer);
		if (NetConnection)
		{
			NetConnection->OwningActor = this;

			UpdateOwningNetConnection();
			UE::Net::FReplicationSystemUtil::UpdateSubObjectGroupMemberships(this);
		}
	}

	UpdateStateInputComponents();

#if ENABLE_VISUAL_LOG
	if (GetLocalRole() == ROLE_Authority && FVisualLogger::Get().IsRecordingOnServer())
	{
		OnServerStartedVisualLogger(true);
	}
#endif

	// notify script that we've been assigned a valid player
	ReceivedPlayer();
}

ULocalPlayer* APlayerController::GetLocalPlayer() const
{
	return Cast<ULocalPlayer>(Player);
}

FPlatformUserId APlayerController::GetPlatformUserId() const
{
	if (const ULocalPlayer* LocalPlayer = GetLocalPlayer())
	{
		return LocalPlayer->GetPlatformUserId();
	}
	return PLATFORMUSERID_NONE;
}

bool APlayerController::IsInViewportClient(UGameViewportClient* ViewportClient) const
{
	const ULocalPlayer* LocalPlayer = GetLocalPlayer();
	if (LocalPlayer && ViewportClient)
	{
		TSharedPtr<const FSlateUser> SlateUser = LocalPlayer->GetSlateUser();
		if (SlateUser && SlateUser->IsWidgetDirectlyUnderCursor(ViewportClient->GetGameViewportWidget()))
		{
			return true;
		}
	}
	return false;
}

int32 APlayerController::GetInputIndex() const
{
	if (ULocalPlayer* LocalPlayer = Cast<ULocalPlayer>(Player))
	{
		return LocalPlayer->GetControllerId();
	}
	return INVALID_CONTROLLERID;
}

void APlayerController::TickPlayerInput(const float DeltaSeconds, const bool bGamePaused)
{
	SCOPE_CYCLE_COUNTER(STAT_PC_TickInput);

	check(PlayerInput);
	PlayerInput->Tick(DeltaSeconds);

	if (ULocalPlayer* LocalPlayer = Cast<ULocalPlayer>(Player))
	{
		if (bEnableMouseOverEvents)
		{
			FVector2D MousePosition;
			FHitResult HitResult;
			bool bHit = false;
			
			UGameViewportClient* ViewportClient = LocalPlayer->ViewportClient;

			// Only send mouse hit events if we're directly over the viewport.
			if ( IsInViewportClient(ViewportClient) )
			{
				if ( ViewportClient->GetMousePosition(MousePosition) )
				{
					bHit = GetHitResultAtScreenPosition(MousePosition, CurrentClickTraceChannel, true, /*out*/ HitResult);
				}
			}

			UPrimitiveComponent* PreviousComponent = CurrentClickablePrimitive.Get();
			UPrimitiveComponent* CurrentComponent = (bHit ? HitResult.Component.Get() : NULL);

			UPrimitiveComponent::DispatchMouseOverEvents(PreviousComponent, CurrentComponent);

			CurrentClickablePrimitive = CurrentComponent;
		}

		if (bEnableTouchOverEvents)
		{
			for (int32 TouchIndexInt = 0; TouchIndexInt < EKeys::NUM_TOUCH_KEYS; ++TouchIndexInt)
			{
				const ETouchIndex::Type FingerIndex = ETouchIndex::Type(TouchIndexInt);

				FHitResult HitResult;
				const bool bHit = GetHitResultUnderFinger(FingerIndex, CurrentClickTraceChannel, true, /*out*/ HitResult);

				UPrimitiveComponent* PreviousComponent = CurrentTouchablePrimitives[TouchIndexInt].Get();
				UPrimitiveComponent* CurrentComponent = (bHit ? HitResult.Component.Get() : NULL);

				const FTouchId TouchId(PlayerInput->GetTouchStates()[FingerIndex].DeviceId, FingerIndex);
				UPrimitiveComponent::DispatchTouchOverEvents(TouchId, this, HitResult, PreviousComponent, CurrentComponent);

				CurrentTouchablePrimitives[TouchIndexInt] = CurrentComponent;
			}
		}
	}

	ProcessPlayerInput(DeltaSeconds, bGamePaused);
	ProcessForceFeedbackAndHaptics(DeltaSeconds, bGamePaused);
}

void APlayerController::TickActor( float DeltaSeconds, ELevelTick TickType, FActorTickFunction& ThisTickFunction )
{
	CSV_SCOPED_TIMING_STAT_EXCLUSIVE(PlayerControllerTick);
	SCOPE_CYCLE_COUNTER(STAT_PlayerControllerTick);
	SCOPE_CYCLE_COUNTER(STAT_PC_TickActor);

	if (TickType == LEVELTICK_PauseTick && !ShouldPerformFullTickWhenPaused())
	{
		if (PlayerInput)
		{
			TickPlayerInput(DeltaSeconds, true);
		}

		// Clear axis inputs from previous frame.
		RotationInput = FRotator::ZeroRotator;

		if (IsValidChecked(this))
		{
			Tick(DeltaSeconds);	// perform any tick functions unique to an actor subclass
		}

		return; //root of tick hierarchy
	}

	//root of tick hierarchy

	const bool bIsClient = IsNetMode(NM_Client);
	const bool bIsLocallyControlled = IsLocalPlayerController();

	if ((GetRemoteRole() == ROLE_AutonomousProxy) && !bIsClient && !bIsLocallyControlled)
	{
		// force physics update for clients that aren't sending movement updates in a timely manner 
		// this prevents cheats associated with artificially induced ping spikes
		// skip updates if pawn lost autonomous proxy role (e.g. TurnOff() call)
		if (IsValid(GetPawn()) && GetPawn()->GetRemoteRole() == ROLE_AutonomousProxy && GetPawn()->IsReplicatingMovement())
		{
			UMovementComponent* PawnMovement = GetPawn()->GetMovementComponent();
			INetworkPredictionInterface* NetworkPredictionInterface = Cast<INetworkPredictionInterface>(PawnMovement);
			if (NetworkPredictionInterface && IsValid(PawnMovement->UpdatedComponent))
			{
				FNetworkPredictionData_Server* ServerData = NetworkPredictionInterface->HasPredictionData_Server() ? NetworkPredictionInterface->GetPredictionData_Server() : nullptr;
				if (ServerData)
				{
					UWorld* World = GetWorld();
					if (ServerData->ServerTimeStamp != 0.f)
					{
						const float WorldTimeStamp = World->GetTimeSeconds();
						const float TimeSinceUpdate = WorldTimeStamp - ServerData->ServerTimeStamp;
						const float PawnTimeSinceUpdate = TimeSinceUpdate * GetPawn()->CustomTimeDilation;
						// See how long we wait to force an update. Setting MAXCLIENTUPDATEINTERVAL to zero allows the server to disable this feature.
						const AGameNetworkManager* GameNetworkManager = (const AGameNetworkManager*)(AGameNetworkManager::StaticClass()->GetDefaultObject());
						const float ForcedUpdateInterval = GameNetworkManager->MAXCLIENTUPDATEINTERVAL;
						const float ForcedUpdateMaxDuration = FMath::Min(GameNetworkManager->MaxClientForcedUpdateDuration, 5.0f);

						// If currently resolving forced updates, and exceeded max duration, then wait for a valid update before enabling them again.
						ServerData->bForcedUpdateDurationExceeded = false;
						if (ServerData->bTriggeringForcedUpdates)
						{
							if (ServerData->ServerTimeStamp > ServerData->ServerTimeLastForcedUpdate)
							{
								// An update came in that was not a forced update (ie a real move), since ServerTimeStamp advanced outside this code.
								UE_LOGF(LogNetPlayerMovement, Log, "Movement detected, resetting forced update state (ServerTimeStamp %.6f > ServerTimeLastForcedUpdate %.6f) Pawn: %ls", ServerData->ServerTimeStamp, ServerData->ServerTimeLastForcedUpdate, *GetNameSafe(GetPawn()));
								ServerData->ResetForcedUpdateState();
							}
							else
							{
								const float PawnTimeSinceForcingUpdates = (WorldTimeStamp - ServerData->ServerTimeBeginningForcedUpdates) * GetPawn()->CustomTimeDilation;
								const float PawnTimeForcedUpdateMaxDuration = ForcedUpdateMaxDuration * GetPawn()->GetActorTimeDilation();

								if (PawnTimeSinceForcingUpdates > PawnTimeForcedUpdateMaxDuration)
								{
									// Waiting for ServerTimeStamp to advance from a client move.
									UE_LOGF(LogNetPlayerMovement, Log, "Setting bForcedUpdateDurationExceeded=true (PawnTimeSinceForcingUpdates %.6f > PawnTimeForcedUpdateMaxDuration %.6f) (bLastRequestNeedsForcedUpdates:%d) Pawn: %ls", PawnTimeSinceForcingUpdates, PawnTimeForcedUpdateMaxDuration, (int32)ServerData->bLastRequestNeedsForcedUpdates, *GetNameSafe(GetPawn()));
									ServerData->bForcedUpdateDurationExceeded = true;
								}
							}
						}
						
						const float CurrentRealTime = World->GetRealTimeSeconds();
						const bool bHitch = (CurrentRealTime - LastMovementUpdateTime) > GameNetworkManager->ServerForcedUpdateHitchThreshold && (LastMovementUpdateTime != 0);
						LastMovementHitch = bHitch ? CurrentRealTime : LastMovementHitch;
						const bool bRecentHitch = bHitch || (CurrentRealTime - LastMovementHitch < GameNetworkManager->ServerForcedUpdateHitchCooldown);
						LastMovementUpdateTime = CurrentRealTime;

						// Trigger forced update if allowed
						const float PawnTimeMinForcedUpdateInterval = (DeltaSeconds + 0.06f) * GetPawn()->CustomTimeDilation;
						const float PawnTimeForcedUpdateInterval = FMath::Max<float>(PawnTimeMinForcedUpdateInterval, ForcedUpdateInterval * GetPawn()->GetActorTimeDilation());

						if (!bRecentHitch && ForcedUpdateInterval > 0.f && (PawnTimeSinceUpdate > PawnTimeForcedUpdateInterval))
						{
							//UE_LOGF(LogPlayerController, Warning, "ForcedMovementTick. PawnTimeSinceUpdate: %f, DeltaSeconds: %f, DeltaSeconds+: %f", PawnTimeSinceUpdate, DeltaSeconds, DeltaSeconds+0.06f);
							const USkeletalMeshComponent* PawnMesh = GetPawn()->FindComponentByClass<USkeletalMeshComponent>();
							const bool bShouldForceUpdate = !ServerData->bForcedUpdateDurationExceeded || ServerData->bLastRequestNeedsForcedUpdates;
							if (bShouldForceUpdate && (!PawnMesh || !PawnMesh->IsSimulatingPhysics()))
							{
								const bool bDidUpdate = NetworkPredictionInterface->ForcePositionUpdate(PawnTimeSinceUpdate);

								// Refresh this pointer in case it has changed (which can happen if character is destroyed or repossessed).
								ServerData = NetworkPredictionInterface->HasPredictionData_Server() ? NetworkPredictionInterface->GetPredictionData_Server() : nullptr;

								if (bDidUpdate && ServerData)
								{
									ServerData->ServerTimeLastForcedUpdate = WorldTimeStamp;

									// Detect initial conditions triggering forced updates.
									if (!ServerData->bTriggeringForcedUpdates)
									{
										ServerData->ServerTimeBeginningForcedUpdates = ServerData->ServerTimeStamp;
										ServerData->bTriggeringForcedUpdates = true;
									}

									// Set server timestamp, if there was movement.
									ServerData->ServerTimeStamp = WorldTimeStamp;
								}
							}
						}
					}
					else
					{
						// If timestamp is zero, set to current time so we don't have a huge initial delta time for correction.
						ServerData->ServerTimeStamp = World->GetTimeSeconds();
						ServerData->ResetForcedUpdateState();
					}
				}
			}
		}

		// update viewtarget replicated info
		if (PlayerCameraManager != nullptr)
		{
			APawn* TargetPawn = PlayerCameraManager->GetViewTargetPawn();
			
			if ((TargetPawn != GetPawn()) && (TargetPawn != nullptr))
			{
				SetTargetViewRotation(TargetPawn->GetViewRotation());
			}
		}
	}
	else if (GetLocalRole() > ROLE_SimulatedProxy)
	{
		// Process PlayerTick with input.
		if (!PlayerInput && (Player == nullptr || Cast<ULocalPlayer>( Player ) != nullptr))
		{
			InitInputSystem();
		}

		if (PlayerInput)
		{
			QUICK_SCOPE_CYCLE_COUNTER(PlayerTick);
			PlayerTick(DeltaSeconds);
		}

		if (!IsValidChecked(this))
		{
			return;
		}

		// update viewtarget replicated info
		if (PlayerCameraManager != nullptr)
		{
			APawn* TargetPawn = PlayerCameraManager->GetViewTargetPawn();
			if ((TargetPawn != GetPawn()) && (TargetPawn != nullptr))
			{
				SmoothTargetViewRotation(TargetPawn, DeltaSeconds);
			}

			// Send a camera update if necessary.
			// That position will be used as the base for replication
			// (i.e., the origin that will be used when calculating NetCullDistance for other Actors / Objects).
			// We only do this when the Pawn will move, to prevent spamming RPCs.
			if (bIsClient && bIsLocallyControlled && GetPawn() && PlayerCameraManager->bUseClientSideCameraUpdates)
			{
				UPawnMovementComponent* PawnMovement = GetPawn()->GetMovementComponent();
				if (PawnMovement != nullptr &&
					!PawnMovement->IsMoveInputIgnored() &&
					(PawnMovement->GetLastInputVector() != FVector::ZeroVector || PawnMovement->Velocity != FVector::ZeroVector))
				{
					PlayerCameraManager->bShouldSendClientSideCameraUpdate = true;
				}
			}
		}
	}

	if (IsValidChecked(this))
	{
		QUICK_SCOPE_CYCLE_COUNTER(Tick);
		Tick(DeltaSeconds);	// perform any tick functions unique to an actor subclass
	}

	// Clear old axis inputs since we are done with them. 
	RotationInput = FRotator::ZeroRotator;

	if (bIsClient && UPhysicsSettings::Get()->PhysicsPrediction.bEnablePhysicsPrediction && GetLocalRole() == ROLE_AutonomousProxy)
	{
		if (UWorld* World = GetWorld())
		{
			if (FPhysScene* PhysScene = World->GetPhysicsScene())
			{
				if (Chaos::FPhysicsSolver* Solver = PhysScene->GetSolver())
				{
					if (Solver->IsUsingFixedDt())
					{
						TickOffsetSyncCountdown += DeltaSeconds;
						UpdateServerAsyncPhysicsTickOffset();
					}
				}
			}
		}
	}

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	if (CheatManager != nullptr)
	{
		CheatManager->TickCollisionDebug();
	}
#endif // !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
}

bool APlayerController::IsNetRelevantFor(const AActor* RealViewer, const AActor* ViewTarget, const FVector& SrcLocation) const
{
	return ( this==RealViewer );
}

void APlayerController::ClientTravel(const FString& URL, ETravelType TravelType, bool bSeamless, FGuid MapPackageGuid)
{
	// Keep track of seamless travel serverside
	if (bSeamless && TravelType == TRAVEL_Relative)
	{
		SeamlessTravelCount++;
	}

	// Now pass on to the RPC
	ClientTravelInternal(URL, TravelType, bSeamless, MapPackageGuid);
}

/// @cond DOXYGEN_WARNINGS

void APlayerController::ClientTravelInternal_Implementation(const FString& URL, ETravelType TravelType, bool bSeamless, FGuid MapPackageGuid)
{
	UWorld* World = GetWorld();

	// Warn the client.
	PreClientTravel(URL, TravelType, bSeamless);

	if (bSeamless && TravelType == TRAVEL_Relative)
	{
		World->SeamlessTravel(URL);
	}
	else
	{
		if (bSeamless)
		{
			UE_LOGF(LogPlayerController, Warning, "Unable to perform seamless travel because TravelType was %i, not TRAVEL_Relative", int32(TravelType));
		}
		// Do the travel.
		GEngine->SetClientTravel(World, *URL, (ETravelType)TravelType);
	}
}

/// @endcond

FString APlayerController::GetPlayerNetworkAddress()
{
	if( Player && Player->IsA(UNetConnection::StaticClass()) )
		return Cast<UNetConnection>(Player)->LowLevelGetRemoteAddress();
	else
		return TEXT("");
}

FString APlayerController::GetServerNetworkAddress()
{
	UNetDriver* NetDriver = NULL;
	if (UWorld* World = GetWorld())
	{
		NetDriver = World->GetNetDriver();
	}

	if( NetDriver && NetDriver->ServerConnection )
	{
		return NetDriver->ServerConnection->LowLevelGetRemoteAddress();
	}

	return TEXT("");
}

bool APlayerController::DefaultCanUnpause()
{
	return GetWorldSettings() != NULL && GetWorldSettings()->GetPauserPlayerState() == PlayerState;
}

void APlayerController::StartSpectatingOnly()
{
	ChangeState(NAME_Spectating);
	PlayerState->SetIsSpectator(true);
	PlayerState->SetIsOnlyASpectator(true);
	bPlayerIsWaiting = false; // Can't spawn, we are only allowed to be a spectator.
}

void APlayerController::EndPlayingState()
{
	if ( GetPawn() != NULL )
	{
		GetPawn()->SetRemoteViewPitch( 0.f );
	}
}


void APlayerController::BeginSpectatingState()
{
	if (GetPawn() != NULL && GetLocalRole() == ROLE_Authority && ShouldKeepCurrentPawnUponSpectating() == false)
	{
		UnPossess();
	}

	DestroySpectatorPawn();
	SetSpectatorPawn(SpawnSpectatorPawn());
}

void APlayerController::SetSpectatorPawn(class ASpectatorPawn* NewSpectatorPawn)
{
	if (IsInState(NAME_Spectating))
	{
		RemovePawnTickDependency(SpectatorPawn);
		SpectatorPawn = NewSpectatorPawn;
		
		if (NewSpectatorPawn)
		{
			// setting to a new valid spectator pawn
			AttachToPawn(NewSpectatorPawn);
			AddPawnTickDependency(NewSpectatorPawn);
			AutoManageActiveCameraTarget(NewSpectatorPawn);
		}
		else
		{
			// clearing the spectator pawn, try to attach to the regular pawn
			APawn* const MyPawn = GetPawn();
			AttachToPawn(MyPawn);
			AddPawnTickDependency(MyPawn);
			if (MyPawn)
			{
				AutoManageActiveCameraTarget(MyPawn);
			}
			else
			{
				AutoManageActiveCameraTarget(this);
			}
		}
	}
}

ASpectatorPawn* APlayerController::SpawnSpectatorPawn()
{
	ASpectatorPawn* SpawnedSpectator = nullptr;

	// Only spawned for the local player
	if ((GetSpectatorPawn() == nullptr) && IsLocalController())
	{
		UWorld* World = GetWorld();
		if (AGameStateBase const* const GameState = World->GetGameState())
		{
			if (UClass* SpectatorClass = GameState->SpectatorClass)
			{
				FActorSpawnParameters SpawnParams;
				SpawnParams.Owner = this;
				SpawnParams.SpawnCollisionHandlingOverride = ESpawnActorCollisionHandlingMethod::AlwaysSpawn;

				SpawnParams.ObjectFlags |= RF_Transient;	// We never want to save spectator pawns into a map
				SpawnedSpectator = World->SpawnActor<ASpectatorPawn>(SpectatorClass, GetSpawnLocation(), GetControlRotation(), SpawnParams);
				if (SpawnedSpectator)
				{
					SpawnedSpectator->SetReplicates(false); // Client-side only
					SpawnedSpectator->PossessedBy(this);
					SpawnedSpectator->DispatchRestart(true);
					if (SpawnedSpectator->PrimaryActorTick.bStartWithTickEnabled)
					{
						SpawnedSpectator->SetActorTickEnabled(true);
					}

					UE_LOGF(LogPlayerController, Verbose, "Spawned spectator %ls [server:%d]", *GetNameSafe(SpawnedSpectator), GetNetMode() < NM_Client);
				}
				else
				{
					UE_LOGF(LogPlayerController, Warning, "Failed to spawn spectator with class %ls", *GetNameSafe(SpectatorClass));
				}
			}
		}
		else
		{
			// This normally happens on clients if the Player is replicated but the GameState has not yet.
			UE_LOGF(LogPlayerController, Verbose, "NULL GameState when trying to spawn spectator!");
		}
	}

	return SpawnedSpectator;
}


void APlayerController::DestroySpectatorPawn()
{
	if (GetSpectatorPawn())
	{
		if (GetViewTarget() == GetSpectatorPawn())
		{
			SetViewTarget(this);
		}

		GetWorld()->DestroyActor(GetSpectatorPawn());
		SetSpectatorPawn(NULL);
	}
}


APawn* APlayerController::GetPawnOrSpectator() const
{
	return GetPawn() ? GetPawn() : GetSpectatorPawn();
}


void APlayerController::UpdateStateInputComponents()
{
	// update Inactive state component
	if (StateName == NAME_Inactive && IsLocalController())
	{
		if (InactiveStateInputComponent == NULL)
		{
			static const FName InactiveStateInputComponentName("PC_InactiveStateInputComponent0");
			InactiveStateInputComponent = NewObject<UInputComponent>(this, UInputSettings::GetDefaultInputComponentClass(), InactiveStateInputComponentName);
			SetupInactiveStateInputComponent(InactiveStateInputComponent);
			InactiveStateInputComponent->RegisterComponent();
			PushInputComponent(InactiveStateInputComponent);
		}
	}
	else if (InactiveStateInputComponent)
	{
		PopInputComponent(InactiveStateInputComponent);
		InactiveStateInputComponent->DestroyComponent();
		InactiveStateInputComponent = NULL;
	}
}

void APlayerController::ChangeState(FName NewState)
{
	if(NewState != StateName)
	{
		// end current state
		if(StateName == NAME_Spectating)
		{
			EndSpectatingState();
		}
		else if(StateName == NAME_Playing)
		{
			EndPlayingState();
		}

		Super::ChangeState(NewState); // Will set StateName, also handles EndInactiveState/BeginInactiveState

		// start new state
		if(StateName == NAME_Playing)
		{
			BeginPlayingState();
		}
		else if (StateName == NAME_Spectating)
		{
			BeginSpectatingState();
		}

		UpdateStateInputComponents();
	}
}

void APlayerController::BeginPlayingState()
{
}

void APlayerController::EndSpectatingState()
{
	if ( PlayerState != NULL )
	{
		if ( PlayerState->IsOnlyASpectator() )
		{
			UE_LOGF(LogPlayerController, Warning, "Spectator only UPlayer* leaving spectating state");
		}
		PlayerState->SetIsSpectator(false);
	}

	bPlayerIsWaiting = false;

	DestroySpectatorPawn();
}

void APlayerController::BeginInactiveState()
{
	if ( (GetPawn() != nullptr) && (GetPawn()->GetController() == this) )
	{
		GetPawn()->SetController(nullptr);
		if (UE::Gameplay::CVars::bAlwaysNotifyClientOnControllerChange)
		{
			if (HasAuthority())
			{
				// OnRep is not called on the server so call notify directly
				GetPawn()->NotifyControllerChanged();
			}
			else
			{
				GetPawn()->OnRep_Controller();
			}
		}
	}
	SetPawn(nullptr);

	GetWorldTimerManager().SetTimer(TimerHandle_UnFreeze, this, &APlayerController::UnFreeze, GetMinRespawnDelay());
}

float APlayerController::GetMinRespawnDelay()
{
	AGameStateBase const* const GameState = GetWorld()->GetGameState();
	
	if (GameState)
	{
		return GameState->GetPlayerRespawnDelay(this);
	}
	return 1.0f;
}

void APlayerController::EndInactiveState()
{
}

void APlayerController::SetupInactiveStateInputComponent(UInputComponent* InComponent)
{
	check(InComponent);

	InComponent->BindAxis("Spectator_Turn", this, &APlayerController::AddYawInput);
	InComponent->BindAxis("Spectator_LookUp", this, &APlayerController::AddPitchInput);
}


void APlayerController::PushInputComponent(UInputComponent* InInputComponent)
{
	if (InInputComponent)
	{
		bool bPushed = false;
		CurrentInputStack.RemoveSingle(InInputComponent);
		for (int32 Index = CurrentInputStack.Num() - 1; Index >= 0; --Index)
		{
			UInputComponent* IC = CurrentInputStack[Index].Get();
			if (IC == nullptr)
			{
				CurrentInputStack.RemoveAt(Index);
			}
			else if (IC->Priority <= InInputComponent->Priority)
			{
				CurrentInputStack.Insert(InInputComponent, Index + 1);
				bPushed = true;
				break;
			}
		}
		if (!bPushed)
		{
			CurrentInputStack.Insert(InInputComponent, 0);
		}
	}
}

bool APlayerController::PopInputComponent(UInputComponent* InInputComponent)
{
	if (InInputComponent)
	{
		if (CurrentInputStack.RemoveSingle(InInputComponent) > 0)
		{
			InInputComponent->ClearBindingValues();
			return true;
		}
	}
	return false;
}

bool APlayerController::IsInputComponentInStack(const UInputComponent* InInputComponent) const
{
	return InInputComponent && CurrentInputStack.Contains(InInputComponent);
}

void APlayerController::AddPitchInput(float Val)
{	
	RotationInput.Pitch += !IsLookInputIgnored() ? Val * (GetDefault<UInputSettings>()->bEnableLegacyInputScales ? InputPitchScale_DEPRECATED : 1.0f) : 0.0f;
}

void APlayerController::AddYawInput(float Val)
{
	RotationInput.Yaw += !IsLookInputIgnored() ? Val * (GetDefault<UInputSettings>()->bEnableLegacyInputScales ? InputYawScale_DEPRECATED : 1.0f) : 0.0f;
}

void APlayerController::AddRollInput(float Val)
{
	RotationInput.Roll += !IsLookInputIgnored() ? Val * (GetDefault<UInputSettings>()->bEnableLegacyInputScales ? InputRollScale_DEPRECATED : 1.0f) : 0.0f;
}

bool APlayerController::IsInputKeyDown(const FKey Key) const
{
	return (PlayerInput ? PlayerInput->IsPressed(Key) : false);
}

bool APlayerController::WasInputKeyJustPressed(const FKey Key) const
{
	return (PlayerInput ? PlayerInput->WasJustPressed(Key) : false);
}

bool APlayerController::WasInputKeyJustReleased(const FKey Key) const
{
	return (PlayerInput ? PlayerInput->WasJustReleased(Key) : false);
}

float APlayerController::GetInputAnalogKeyState(const FKey Key) const
{
	return (PlayerInput ? PlayerInput->GetKeyValue(Key) : 0.f);
}

FVector APlayerController::GetInputVectorKeyState(const FKey Key) const
{
	return (PlayerInput ? PlayerInput->GetRawVectorKeyValue(Key) : FVector());
}

void APlayerController::GetInputTouchState(ETouchIndex::Type FingerIndex, float& LocationX, float& LocationY, bool& bIsCurrentlyPressed) const
{
	using namespace UE;

	if (PlayerInput)
	{
		if (FingerIndex < EKeys::NUM_TOUCH_KEYS)
		{
			const FTouchState& State = PlayerInput->GetTouchStates()[FingerIndex];
			LocationX = State.ViewportRelativeLocation.X;
			LocationY = State.ViewportRelativeLocation.Y;
			bIsCurrentlyPressed = State.Pressure != 0;
		}
		else
		{
			bIsCurrentlyPressed = false;
			UE_LOGF(LogPlayerController, Warning, "Requesting information for invalid finger index.");
		}
	}
	else
	{
		LocationX = LocationY = 0.f;
		bIsCurrentlyPressed = false;
	}
}
void APlayerController::GetInputTouchState(ETouchIndex::Type FingerIndex, double& LocationX, double& LocationY, bool& bIsCurrentlyPressed) const
{
	float X = (float)LocationX, Y = (float)LocationY;
	GetInputTouchState(FingerIndex, X, Y, bIsCurrentlyPressed);
	LocationX = X;
	LocationY = Y;
}

void APlayerController::GetInputMotionState(FVector& Tilt, FVector& RotationRate, FVector& Gravity, FVector& Acceleration) const
{
	Tilt = GetInputVectorKeyState(EKeys::Tilt);
	RotationRate = GetInputVectorKeyState(EKeys::RotationRate);
	Gravity = GetInputVectorKeyState(EKeys::Gravity);
	Acceleration = GetInputVectorKeyState(EKeys::Acceleration);
}

float APlayerController::GetInputKeyTimeDown(const FKey Key) const
{
	return (PlayerInput ? PlayerInput->GetTimeDown(Key) : 0.f);
}

bool APlayerController::GetMousePosition(float& LocationX, float& LocationY) const
{
	bool bGotMousePosition = false;
	ULocalPlayer* LocalPlayer = Cast<ULocalPlayer>(Player);

	if (LocalPlayer && LocalPlayer->ViewportClient)
	{
		FVector2D MousePosition;
		
		bGotMousePosition = LocalPlayer->ViewportClient->GetMousePosition(MousePosition);

		if (bGotMousePosition)
		{
			LocationX = MousePosition.X;
			LocationY = MousePosition.Y;
		}
	}

	return bGotMousePosition;
}
bool APlayerController::GetMousePosition(double& LocationX, double& LocationY) const
{
	float X, Y;
	if(GetMousePosition(X, Y))
	{
		LocationX = X;
		LocationY = Y;
		return true;
	}
	return false;
}

void APlayerController::GetInputMouseDelta(float& DeltaX, float& DeltaY) const
{
	if (PlayerInput)
	{
		DeltaX = PlayerInput->GetKeyValue(EKeys::MouseX);
		DeltaY = PlayerInput->GetKeyValue(EKeys::MouseY);
	}
	else
	{
		DeltaX = DeltaY = 0.f;
	}
}

void APlayerController::GetInputMouseDelta(double& DeltaX, double& DeltaY) const
{
	float DX, DY;
	GetInputMouseDelta(DX, DY);
	DeltaX = DX;
	DeltaY = DY;
}

void APlayerController::GetInputAnalogStickState(EControllerAnalogStick::Type WhichStick, float& StickX, float& StickY) const
{
	if (PlayerInput)
	{
		switch (WhichStick)
		{
		case EControllerAnalogStick::CAS_LeftStick:
			StickX = PlayerInput->GetKeyValue(EKeys::Gamepad_LeftX);
			StickY = PlayerInput->GetKeyValue(EKeys::Gamepad_LeftY);
			break;

		case EControllerAnalogStick::CAS_RightStick:
			StickX = PlayerInput->GetKeyValue(EKeys::Gamepad_RightX);
			StickY = PlayerInput->GetKeyValue(EKeys::Gamepad_RightY);
			break;

		default:
			StickX = 0.f;
			StickY = 0.f;
		}
	}
	else
	{
		StickX = StickY = 0.f;
	}
}
void APlayerController::GetInputAnalogStickState(EControllerAnalogStick::Type WhichStick, double& StickX, double& StickY) const
{
	float DX, DY;
	GetInputAnalogStickState(WhichStick, DX, DY);
	StickX = DX;
	StickY = DY;
}

void APlayerController::EnableInput(class APlayerController* PlayerController)
{
	if (PlayerController == this || PlayerController == NULL)
	{
		bInputEnabled = true;
	}
	else
	{
		UE_LOGF(LogPlayerController, Error, "EnableInput can only be specified on a PlayerController for itself");
	}
}

void APlayerController::DisableInput(class APlayerController* PlayerController)
{
	if (PlayerController == this || PlayerController == NULL)
	{
		bInputEnabled = false;
	}
	else
	{
		UE_LOGF(LogPlayerController, Error, "DisableInput can only be specified on a PlayerController for itself");
	}
}


void APlayerController::ActivateTouchInterface(UTouchInterface* NewTouchInterface)
{
	CurrentTouchInterface = NewTouchInterface;
	if(NewTouchInterface)
	{
		if (!VirtualJoystick.IsValid())
		{
			CreateTouchInterface();
		}
		else
		{
			NewTouchInterface->Activate(VirtualJoystick);
		}
	}
	else if (VirtualJoystick.IsValid())
	{
		ULocalPlayer* LocalPlayer = Cast<ULocalPlayer>(Player);
		if (LocalPlayer && LocalPlayer->ViewportClient)
		{
			LocalPlayer->ViewportClient->RemoveViewportWidgetContent(VirtualJoystick.ToSharedRef());
		}
		//clear any input before clearing the VirtualJoystick
		FlushPressedKeys();
		VirtualJoystick = NULL;
	}
}

void APlayerController::SetVirtualJoystickVisibility(bool bVisible)
{
	if (VirtualJoystick.IsValid())
	{
		VirtualJoystick->SetJoystickVisibility(bVisible, false);
	}
}

void FInputModeDataBase::SetFocusAndLocking(FReply& SlateOperations, TSharedPtr<SWidget> InWidgetToFocus, bool bLockMouseToViewport, TSharedRef<SViewport> InViewportWidget) const
{
	if (InWidgetToFocus.IsValid())
	{
		SlateOperations.SetUserFocus(InWidgetToFocus.ToSharedRef());
	}

	if (bLockMouseToViewport)
	{	
		SlateOperations.LockMouseToWidget(InViewportWidget);
	}
	else
	{
		SlateOperations.ReleaseMouseLock();
	}
}

#if UE_ENABLE_DEBUG_DRAWING
const FString& FInputModeDataBase::GetDebugDisplayName() const
{
	static const FString DisplayName = "Base";
	return DisplayName;
}
#endif	// UE_ENABLE_DEBUG_DRAWING

FInputModeUIOnly::~FInputModeUIOnly() = default;

FInputModeUIOnly& FInputModeUIOnly::SetWidgetToFocus(TSharedPtr<SWidget> InWidgetToFocus)
{
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	if (InWidgetToFocus.IsValid() && !InWidgetToFocus->SupportsKeyboardFocus())
	{
		UE_LOGF(LogPlayerController, Error, "InputMode:UIOnly - Attempting to focus Non-Focusable widget %ls!", *InWidgetToFocus->ToString());
	}
#endif
	WidgetToFocus = InWidgetToFocus;
	return *this;
}

FInputModeUIOnly::FInputModeUIOnly()
	: WidgetToFocus()
	, MouseLockMode(EMouseLockMode::LockInFullscreen)
{
}

FInputModeUIOnly& FInputModeUIOnly::SetLockMouseToViewportBehavior(EMouseLockMode InMouseLockMode)
{
	MouseLockMode = InMouseLockMode;
	return *this;
}

#if UE_ENABLE_DEBUG_DRAWING
const FString& FInputModeUIOnly::GetDebugDisplayName() const
{
	static const FString DebugName = "UI Only (Input will only be consumed by the UI, not the player!)";
	return DebugName;
}
#endif	// UE_ENABLE_DEBUG_DRAWING

void FInputModeUIOnly::ApplyInputMode(FReply& SlateOperations, class UGameViewportClient& GameViewportClient) const
{
	TSharedPtr<SViewport> ViewportWidget = GameViewportClient.GetGameViewportWidget();
	if (ViewportWidget.IsValid())
	{
		const bool bLockMouseToViewport = MouseLockMode == EMouseLockMode::LockAlways
												 || (MouseLockMode == EMouseLockMode::LockInFullscreen && GameViewportClient.IsExclusiveFullscreenViewport());
		SetFocusAndLocking(SlateOperations, WidgetToFocus, bLockMouseToViewport, ViewportWidget.ToSharedRef());

		SlateOperations.ReleaseMouseCapture();

		GameViewportClient.SetMouseLockMode(MouseLockMode);
		GameViewportClient.SetIgnoreInput(true);
		GameViewportClient.SetMouseCaptureMode(EMouseCaptureMode::NoCapture);
	}
}

FInputModeGameAndUI::FInputModeGameAndUI()
	: WidgetToFocus()
	, MouseLockMode(EMouseLockMode::DoNotLock)
	, bHideCursorDuringCapture(true)
{
}

FInputModeGameAndUI::~FInputModeGameAndUI() = default;

void FInputModeGameAndUI::ApplyInputMode(FReply& SlateOperations, class UGameViewportClient& GameViewportClient) const
{
	TSharedPtr<SViewport> ViewportWidget = GameViewportClient.GetGameViewportWidget();
	if (ViewportWidget.IsValid())
	{
		const bool bLockMouseToViewport = MouseLockMode == EMouseLockMode::LockAlways
			|| (MouseLockMode == EMouseLockMode::LockInFullscreen && GameViewportClient.IsExclusiveFullscreenViewport());
		SetFocusAndLocking(SlateOperations, WidgetToFocus, bLockMouseToViewport, ViewportWidget.ToSharedRef());

		SlateOperations.ReleaseMouseCapture();

		GameViewportClient.SetMouseLockMode(MouseLockMode);
		GameViewportClient.SetIgnoreInput(false);
		GameViewportClient.SetHideCursorDuringCapture(bHideCursorDuringCapture);
		GameViewportClient.SetMouseCaptureMode(EMouseCaptureMode::CaptureDuringMouseDown);
	}
}

#if UE_ENABLE_DEBUG_DRAWING
const FString& FInputModeGameAndUI::GetDebugDisplayName() const
{
	static const FString DisplayName = "Game and UI";
	return DisplayName;
}
#endif	// UE_ENABLE_DEBUG_DRAWING

FInputModeGameOnly::FInputModeGameOnly()
	: bConsumeCaptureMouseDown(true)
{
}

FInputModeGameOnly::~FInputModeGameOnly() = default;

#if UE_ENABLE_DEBUG_DRAWING
const FString& FInputModeGameOnly::GetDebugDisplayName() const
{
	static const FString DisplayName = "Game Only (Input will only be consumed by the player, not UI)";
	return DisplayName;
}
#endif	// UE_ENABLE_DEBUG_DRAWING

void FInputModeGameOnly::ApplyInputMode(FReply& SlateOperations, class UGameViewportClient& GameViewportClient) const
{
	TSharedPtr<SViewport> ViewportWidget = GameViewportClient.GetGameViewportWidget();
	if (ViewportWidget.IsValid())
	{
		TSharedRef<SViewport> ViewportWidgetRef = ViewportWidget.ToSharedRef();
		SlateOperations.UseHighPrecisionMouseMovement(ViewportWidgetRef);
		SlateOperations.SetUserFocus(ViewportWidgetRef);
		SlateOperations.LockMouseToWidget(ViewportWidgetRef);
		GameViewportClient.SetMouseLockMode(EMouseLockMode::LockOnCapture);
		GameViewportClient.SetIgnoreInput(false);
		GameViewportClient.SetMouseCaptureMode(bConsumeCaptureMouseDown ? EMouseCaptureMode::CapturePermanently : EMouseCaptureMode::CapturePermanently_IncludingInitialMouseDown);
	}
}

void APlayerController::SetInputMode(const FInputModeDataBase& InData)
{
	UGameViewportClient* GameViewportClient = GetWorld()->GetGameViewport();
	ULocalPlayer* LocalPlayer = Cast< ULocalPlayer >( Player );
	if ( GameViewportClient && LocalPlayer )
	{
		InData.ApplyInputMode( LocalPlayer->GetSlateOperations(), *GameViewportClient );
		bShouldFlushInputWhenViewportFocusChanges = InData.ShouldFlushInputOnViewportFocus();

		// Keep track of the name of this input mode for debug purposes
#if UE_ENABLE_DEBUG_DRAWING
		CurrentInputModeDebugString = InData.GetDebugDisplayName();
#endif
	}
}

#if UE_ENABLE_DEBUG_DRAWING
const FString& APlayerController::GetCurrentInputModeDebugString() const
{
	return CurrentInputModeDebugString;
}
#endif // #if UE_ENABLE_DEBUG_DRAWING

void APlayerController::UpdateCameraManager(float DeltaSeconds)
{
#if UE_SERVER && UE_AUTORTFM && UE_WITH_REMOTE_OBJECT_HANDLE
	if (!AutoRTFM::IsClosed())
	{
		if (PlayerCameraManager.GetResidence() != EResidence::Local)
		{
			UE_LOGF(LogPlayerController, VeryVerbose, "%s: Skipping due to PlayerCameraManager being non-local", __FUNCTION__);
			return;
		}
	}
#endif

	if (PlayerCameraManager)
	{
		PlayerCameraManager->UpdateCamera(DeltaSeconds);
	}
}

void APlayerController::BuildHiddenComponentList(const FVector& ViewLocation, TSet<FPrimitiveComponentId>& HiddenComponentsOut)
{
	// Makes no sens to build hidden component list if should not render any components.
	check(bRenderPrimitiveComponents);

	// Translate the hidden actors list to a hidden primitive list.
	UpdateHiddenActors(ViewLocation);

	for (int32 ActorIndex = 0; ActorIndex < HiddenActors.Num(); ++ActorIndex)
	{
		AActor* HiddenActor = HiddenActors[ActorIndex];
		if (HiddenActor != NULL)
		{
			TInlineComponentArray<UPrimitiveComponent*> Components;
			HiddenActor->GetComponents(Components);

			for (int32 ComponentIndex = 0; ComponentIndex < Components.Num(); ComponentIndex++)
			{
				UPrimitiveComponent* PrimitiveComponent = Components[ComponentIndex];
				if (PrimitiveComponent->IsRegistered())
				{
					HiddenComponentsOut.Add(PrimitiveComponent->GetPrimitiveSceneId());

					for (USceneComponent* AttachedChild : PrimitiveComponent->GetAttachChildren())
					{						
						UPrimitiveComponent* AttachChildPC = Cast<UPrimitiveComponent>(AttachedChild);
						if (AttachChildPC && AttachChildPC->IsRegistered())
						{
							HiddenComponentsOut.Add(AttachChildPC->GetPrimitiveSceneId());
						}
					}
				}
			}
		}
		else
		{
			HiddenActors.RemoveAt(ActorIndex);
			ActorIndex--;
		}
	}

	// iterate backwards so we can remove as we go
	HiddenComponentsOut.Reserve(HiddenComponentsOut.Num() + HiddenPrimitiveComponents.Num());
	for (int32 ComponentIndx = HiddenPrimitiveComponents.Num() - 1; ComponentIndx >= 0; --ComponentIndx)
	{
		if (UPrimitiveComponent* Component = HiddenPrimitiveComponents[ComponentIndx].Get())
		{
			if (Component->IsRegistered())
			{
				HiddenComponentsOut.Add(Component->GetPrimitiveSceneId());
			}
		}
		else
		{
			HiddenPrimitiveComponents.RemoveAtSwap(ComponentIndx);
		}
	}

	// Allow a chance to operate on a per primitive basis
	UpdateHiddenComponents(ViewLocation, HiddenComponentsOut);
}

/// @cond DOXYGEN_WARNINGS

void APlayerController::ClientRepObjRef_Implementation(UObject *Object)
{
	UE_LOGF(LogPlayerController, Warning, "APlayerController::ClientRepObjRef repped: %ls", Object ? *Object->GetName() : TEXT("NULL") );
}

/// @endcond

void FDynamicForceFeedbackDetails::Update(FForceFeedbackValues& Values) const
{
	if (bAffectsLeftLarge)
	{
		Values.LeftLarge = FMath::Clamp(Intensity, Values.LeftLarge, 1.f);
	}
	if (bAffectsLeftSmall)
	{
		Values.LeftSmall = FMath::Clamp(Intensity, Values.LeftSmall, 1.f);
	}
	if (bAffectsRightLarge)
	{
		Values.RightLarge = FMath::Clamp(Intensity, Values.RightLarge, 1.f);
	}
	if (bAffectsRightSmall)
	{
		Values.RightSmall = FMath::Clamp(Intensity, Values.RightSmall, 1.f);
	}
}

/// @cond DOXYGEN_WARNINGS

void APlayerController::OnServerStartedVisualLogger_Implementation(bool bIsLogging)
{
#if ENABLE_VISUAL_LOG
	FVisualLogger::Get().SetIsRecordingOnServer(bIsLogging);
	ClientMessage(FString::Printf(TEXT("Visual Loggger is %s."), FVisualLogger::Get().IsRecordingOnServer() ? TEXT("now recording") : TEXT("disabled")));
#endif
}

/// @endcond

bool APlayerController::ShouldPerformFullTickWhenPaused() const
{
	return bShouldPerformFullTickWhenPaused || 
		(/*bIsInVr =*/GEngine->StereoRenderingDevice.IsValid() && GEngine->StereoRenderingDevice->IsStereoEnabled() && 
			GEngine->XRSystem.IsValid() && GEngine->XRSystem->GetHMDDevice() && GEngine->XRSystem->GetHMDDevice()->IsHMDConnected());
}

void APlayerController::IncludeInNetConditionGroup(FName NetGroup)
{
	checkf(!UE::Net::IsSpecialNetConditionGroup(NetGroup), TEXT("Cannot add a player to special netcondition group %s manually. This group membership is managed by the network engine automatically."), *NetGroup.ToString());
	checkf(!NetGroup.IsNone(), TEXT("Invalid netcondition group: NONE"));
	NetConditionGroups.AddUnique(NetGroup);

	UE::Net::FReplicationSystemUtil::UpdateSubObjectGroupMemberships(this);
}

void APlayerController::RemoveFromNetConditionGroup(FName NetGroup)
{
	NetConditionGroups.RemoveSingleSwap(NetGroup);
	UE::Net::FReplicationSystemUtil::RemoveSubObjectGroupMembership(this, NetGroup);
}


void APlayerController::OnReplicationStartedForIris(const FOnReplicationStartedParams& Params)
{
	using namespace UE::Net;

	Super::OnReplicationStartedForIris(Params);

	// Bump prio of playercontroller in order to make sure it replicates really early
	static constexpr float PlayerControllerStaticPriority = 100.f;
	FReplicationSystemUtil::SetStaticPriority(this, PlayerControllerStaticPriority);

	// Enable groups once owner is set!!
	FReplicationSystemUtil::UpdateSubObjectGroupMemberships(this);
}

void APlayerController::ExecuteAsyncPhysicsCommand(const FAsyncPhysicsTimestamp& AsyncPhysicsTimestamp, UObject* OwningObject, const TFunction<void()>& Command, const bool bEnableResim)
{
	if(UWorld* World = GetWorld())
	{
		if(FPhysScene* PhysScene = World->GetPhysicsScene())
		{
			const int32 PhysicsStep = IsLocalController() ? AsyncPhysicsTimestamp.LocalFrame : AsyncPhysicsTimestamp.ServerFrame;
			PhysScene->EnqueueAsyncPhysicsCommand(PhysicsStep, OwningObject, Command, bEnableResim);
		}
	}
}

FAsyncPhysicsTimestamp APlayerController::GetPhysicsTimestamp()
{
	using namespace Chaos;

	FAsyncPhysicsTimestamp Timestamp;

	if (UWorld* World = GetWorld())
	{
		if (FPhysScene* PhysScene = World->GetPhysicsScene())
		{
			if (FPhysicsSolverBase* Solver = static_cast<FPhysicsSolverBase*>(PhysScene->GetSolver()))
			{
				Timestamp.LocalFrame = Solver->GetMarshallingManager().GetInternalStep_External();
				Timestamp.ServerFrame = Timestamp.LocalFrame + NetworkPhysicsTickOffset; // On the server NetworkPhysicsTickOffset is always 0 so it's safe to always add on both client and server to convert LocalFrame to ServerFrame
			}
		}
	}

	return Timestamp;
}

uint32 APlayerController::GetClientHandshakeId() const
{
	return ClientHandshakeId;
}

void APlayerController::SetClientHandshakeId(uint32 InClientHandshakeId)
{
	ClientHandshakeId = InClientHandshakeId;
}

void APlayerController::SetTickOffsetBufferTime(float NewBufferTime)
{
	TickOffsetBufferTime = FMath::Max(0.0f, NewBufferTime);
};

void APlayerController::UpdateServerAsyncPhysicsTickOffset()
{
	if (NetworkPhysicsCvars::TickOffsetUpdateIntervalTime <= 0 || TickOffsetSyncCountdown < (NetworkPhysicsCvars::TickOffsetUpdateIntervalTime * 0.001f))
	{
		return;
	}
	TickOffsetSyncCountdown = 0.0f;

	FAsyncPhysicsTimestamp Timestamp = GetPhysicsTimestamp();
	Timestamp.ServerFrame = bNetworkPhysicsTickOffsetAssigned ? Timestamp.ServerFrame : INDEX_NONE; // If offset is not yet assigned, set an invalid ServerFrame
	ServerSendLatestAsyncPhysicsTimestamp(Timestamp);
}

void APlayerController::ServerSendLatestAsyncPhysicsTimestamp_Implementation(FAsyncPhysicsTimestamp Timestamp)
{
	ensure(UPhysicsSettings::Get()->PhysicsPrediction.bEnablePhysicsPrediction);

	// Get the fixed timestep from project settings
	float AsyncFixedTimeStepSize = UPhysicsSettings::Get()->AsyncFixedTimeStepSize;

	if (UWorld* World = GetWorld())
	{
		if (FPhysScene* PhysScene = World->GetPhysicsScene())
		{
			if (Chaos::FPhysicsSolver* Solver = PhysScene->GetSolver())
			{
				// Get fixed timestep from solver since it can have been altered
				AsyncFixedTimeStepSize = Solver->GetAsyncDeltaTime();
			}
		}
	}

	if (FMath::IsNearlyZero(AsyncFixedTimeStepSize))
	{
		return;
	}

	// Get current server timestamp and add the frame buffer to the ServerFrame
	FAsyncPhysicsTimestamp ActualTimestamp = GetPhysicsTimestamp();
	const int32 BufferTickSize = FMath::CeilToInt((TickOffsetBufferTime * 0.001f) / AsyncFixedTimeStepSize);
	ActualTimestamp.ServerFrame += BufferTickSize;

	// Mark offset as assigned when we get a valid predicted server frame.
	const int32 PredictedServerFrame = Timestamp.ServerFrame;
	bNetworkPhysicsTickOffsetAssigned |= PredictedServerFrame != INDEX_NONE;

	const float CurrentTime = GetWorld()->GetTimeSeconds();
	const int32 TimestampDiff = FMath::Abs(PredictedServerFrame - ActualTimestamp.ServerFrame);
	const float TimestampTimeDiff = TimestampDiff * AsyncFixedTimeStepSize;
	NetworkPhysicsTickOffsetDesyncAccumulatedTime = (TimestampDiff == 0) ? CurrentTime : NetworkPhysicsTickOffsetDesyncAccumulatedTime;

	// Send update to client if offset is not assigned or over correction limits
	// Note that we are sending the current ServerFrame along with the frame buffer added, to the client.
	if (!bNetworkPhysicsTickOffsetAssigned
		|| TimestampTimeDiff > (NetworkPhysicsCvars::TickOffsetCorrectionSizeTimeLimit * 0.001f)
		|| CurrentTime - NetworkPhysicsTickOffsetDesyncAccumulatedTime > (NetworkPhysicsCvars::TickOffsetCorrectionTimeLimit * 0.001f))
	{
		Timestamp.ServerFrame = ActualTimestamp.ServerFrame;
		NetworkPhysicsTickOffsetDesyncAccumulatedTime = CurrentTime;

#if DEBUG_NETWORK_PHYSICS
		UE_LOGF(LogPlayerController, Log, "APlayerController::ServerSendLatestAsyncPhysicsTimestamp_Implementation. Sync physics tick with client. ClientFrame: %d, ServerFrame: %d, BufferSize: %d, PredictedFrame: %d)"
			, Timestamp.LocalFrame, Timestamp.ServerFrame, BufferTickSize, PredictedServerFrame);
#endif

		ClientSetupNetworkPhysicsTimestamp(Timestamp); /* Reliable RPC */
	}

	/*
	* Use time dilation on client to adjust the frame offset and keep the buffer in check
	* If buffer goes low, speed up the client by raising the time dilation multiplier (each deltaTime accounts for more of the accumulated time, filling the accumulated time faster to tick the next physics step)
	* If buffer goes high, slow down the client by lowering the time dilation multiplier (each deltaTime accounts for less of the accumulated time, taking longer to fill the accumulated time
	*/
	if (bNetworkPhysicsTickOffsetAssigned)
	{
		// Get the buffer offset amount that deviates from the target buffer (Note: the buffer is already added to ActualTimestamp.ServerFrame here and in the PredictedServerFrame received from the client)
		// 0 means buffer is perfect, positive value means the buffer is too large, negative value means the buffer is too small
		int32 CurrentFrameBufferOffset = Timestamp.ServerFrame - ActualTimestamp.ServerFrame;

		if (NetworkPhysicsCvars::TimeDilationEscalation == false)
		{
			CurrentFrameBufferOffset = FMath::Clamp(CurrentFrameBufferOffset, -1, 1);
		}
			
		// Calculate desired dilation and send to client
		const float TimeDilationDecay = FMath::Clamp(1.0f - (NetworkPhysicsCvars::TimeDilationEscalationDecay * FMath::Abs(CurrentFrameBufferOffset)), NetworkPhysicsCvars::TimeDilationEscalationDecayMax, 1.0f);
		float CalculatedTimeDilation = 1.0f + ((NetworkPhysicsCvars::TimeDilationAmount * -CurrentFrameBufferOffset) * TimeDilationDecay);
		CalculatedTimeDilation = FMath::Clamp(CalculatedTimeDilation, NetworkPhysicsCvars::TimeDilationMin, NetworkPhysicsCvars::TimeDilationMax);

		ClientAckTimeDilation(CalculatedTimeDilation, ActualTimestamp.LocalFrame);
	}
	
}

void APlayerController::ClientSetupNetworkPhysicsTimestamp_Implementation(FAsyncPhysicsTimestamp Timestamp)
{
	ensure(UPhysicsSettings::Get()->PhysicsPrediction.bEnablePhysicsPrediction);

	// Assign async physics tick offset
	bNetworkPhysicsTickOffsetAssigned = true;
	NetworkPhysicsTickOffset = Timestamp.ServerFrame - Timestamp.LocalFrame;

#if DEBUG_NETWORK_PHYSICS
	UE_LOGF(LogPlayerController, Log, "APlayerController::ClientSetupNetworkPhysicsTimestamp_Implementation. ClientFrame: %d, ServerFrame: %d, NetworkPhysicsTickOffset: %d)"
		, Timestamp.LocalFrame, Timestamp.ServerFrame, NetworkPhysicsTickOffset);
#endif
}

void APlayerController::ClientAckTimeDilation_Implementation(float TimeDilation, int32 ServerStep)
{
	if (UWorld* World = GetWorld())
	{
		World->GetPhysicsScene()->SetNetworkDeltaTimeScale(TimeDilation);
	}
}

void APlayerController::SetLocalPlayerConnectionIdentifier(int32 Identifier)
{
	LocalPlayerConnectionIdentifier = Identifier;
}

int32 APlayerController::GetLocalPlayerConnectionIdentifier() const
{
	return LocalPlayerConnectionIdentifier;
}

ANoPawnPlayerController::ANoPawnPlayerController(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bReplicates = true;
	SetRole(ROLE_Authority);

	PrimaryActorTick.bCanEverTick = false;

	// This controller's updates don't need to be replicated to the client very frequently because
	// it's only intended to be used by the server for replication, relevancy and establishing the
	// initial connection on the client.
	//
	// This low update frequency means that ForceNetUpdate() must be called when any explicit changes
	// are made to the controller, including being attached or detached from a UNetConnection.
	SetNetUpdateFrequency(0.1f);
}

void ANoPawnPlayerController::PostInitializeComponents()
{
	// Don't call APlayerController::PostInitializeComponents() because it will spawn an instance of APlayerState
	// which doesn't make sense in ANoPawnPlayerController because it doesn't have an associated pawn.

	AController::PostInitializeComponents();
}

AActor* ANoPawnPlayerController::GetViewTarget() const
{
	return const_cast<ANoPawnPlayerController*>(this);
}

class UNetConnection* ANoPawnPlayerController::GetNetConnection() const
{
	return NetConnection;
}

void ANoPawnPlayerController::ServerSetViewTargetPosition_Implementation(FVector ViewTargetPos)
{
	AActor* Actor = Cast<AActor>(this);
	Actor->SetActorLocation(ViewTargetPos);
}

void ANoPawnPlayerController::ServerEnableReplicationToConnection_Implementation(bool bEnable)
{
	UE_LOGF(LogNet, Log, "%ls replication for connections using %ls (previously %ls).",
		   bEnable ? TEXT("Enabled") : TEXT("Disabled"),
		   *GetName(),
		   bEnableReplicationToConnection ? TEXT("enabled") : TEXT("disabled"));

	bEnableReplicationToConnection = bEnable;
}

#if UE_WITH_REMOTE_OBJECT_HANDLE
FRemoteServerId ANoPawnPlayerController::GetPreferredRemoteServer() const
{
	// NoPawnPlayerControllers are server-specific & never migrate servers
	return UE::RemoteObject::Handle::GetOwnerServerId(this);
}
#endif

#undef LOCTEXT_NAMESPACE

