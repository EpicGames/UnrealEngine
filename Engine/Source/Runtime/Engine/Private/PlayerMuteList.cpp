// Copyright Epic Games, Inc. All Rights Reserved.

#include "GameFramework/PlayerMuteList.h"
#include "Engine/World.h"
#include "GameFramework/PlayerController.h"
#include "GameFramework/PlayerState.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PlayerMuteList)

constexpr EVoiceBlockReasons ClientVisibleVoiceFilterReasons = EVoiceBlockReasons::Muted | EVoiceBlockReasons::Gameplay | EVoiceBlockReasons::Blocked;

FString LexToString(const EVoiceBlockReasons& Reason)
{
	FString Result;
	for (EVoiceBlockReasons BlockFlag : TEnumRange<EVoiceBlockReasons>())
	{
		if (EnumHasAnyFlags(Reason, BlockFlag))
		{
			if (!Result.IsEmpty())
			{
				Result += TEXT("|");
			}

			switch (Reason)
			{
			case EVoiceBlockReasons::None:
				Result += TEXT("NONE");
				break;
			case EVoiceBlockReasons::Muted:
				Result += TEXT("MUTED");
				break;
			case EVoiceBlockReasons::Gameplay:
				Result += TEXT("GAMEPLAY");
				break;
			case EVoiceBlockReasons::Blocked:
				Result += TEXT("BLOCKED");
				break;
			case EVoiceBlockReasons::BlockedBy:
				Result += TEXT("BLOCKED BY");
				break;
			default:
				ensureMsgf(false, TEXT("Invalid reason: %u"), static_cast<uint8>(Reason));
				Result += TEXT("invalid");
			}
		}
	}

	return Result;
}

bool FPlayerMuteList::AddVoiceBlockReason(const FUniqueNetIdPtr& PlayerId, EVoiceBlockReasons VoiceBlockReason)
{
	check(VoiceBlockReason != EVoiceBlockReasons::None);

	EVoiceBlockReasons& BlockReasons = VoicePacketFilterMap.FindOrAdd(PlayerId.ToSharedRef());
	const EVoiceBlockReasons OldBlockReasons = BlockReasons;
	EnumAddFlags(BlockReasons, VoiceBlockReason);
	
	return (OldBlockReasons & ClientVisibleVoiceFilterReasons) == EVoiceBlockReasons::None && 
		VoiceBlockReason != EVoiceBlockReasons::BlockedBy;
}

bool FPlayerMuteList::RemoveVoiceBlockReason(const FUniqueNetIdPtr& PlayerId, EVoiceBlockReasons VoiceBlockReason)
{
	check(VoiceBlockReason != EVoiceBlockReasons::None);

	if (EVoiceBlockReasons* BlockReasons = VoicePacketFilterMap.Find(PlayerId.ToSharedRef()))
	{
		const EVoiceBlockReasons OldBlockReasons = *BlockReasons;
		EnumRemoveFlags(*BlockReasons, VoiceBlockReason);

		return (OldBlockReasons & ClientVisibleVoiceFilterReasons) != EVoiceBlockReasons::None && 
			(*BlockReasons & ClientVisibleVoiceFilterReasons) == EVoiceBlockReasons::None;
	}

	return false;
}

void FPlayerMuteList::ServerMutePlayer(APlayerController* OwningPC, const FUniqueNetIdRepl& MuteId)
{
	// Add explicit mute to reasons to block comms.
	if (AddVoiceBlockReason(MuteId.GetUniqueNetId(), EVoiceBlockReasons::Muted))
	{
		// This is the first reason added, so we transitioned unmuted -> muted. Replicate mute state to client
		OwningPC->ClientMutePlayer(MuteId);
	}
}

void FPlayerMuteList::ServerUnmutePlayer(APlayerController* OwningPC, const FUniqueNetIdRepl& UnmuteId)
{
	// Remove explicit mute from reasons to block comms.
	if (RemoveVoiceBlockReason(UnmuteId.GetUniqueNetId(), EVoiceBlockReasons::Muted))
	{
		// We removed the last flag, transitioning from muted -> unmuted. Replicate mute state to client.
		OwningPC->ClientUnmutePlayer(UnmuteId);
	}
}

void FPlayerMuteList::GameplayMutePlayer(APlayerController* OwningPC, const FUniqueNetIdRepl& MuteId)
{
	// Add gameplay to the list of reasons to block comms.
	if (AddVoiceBlockReason(MuteId.GetUniqueNetId(), EVoiceBlockReasons::Gameplay))
	{
		// This is the first reason added, so we transitioned unmuted -> muted. Replicate mute state to client
		OwningPC->ClientMutePlayer(MuteId);
	}
}

void FPlayerMuteList::GameplayUnmutePlayer(APlayerController* OwningPC, const FUniqueNetIdRepl& UnmuteId)
{
	// Remove gameplay from the list of reasons to block comms.
	if (RemoveVoiceBlockReason(UnmuteId.GetUniqueNetId(), EVoiceBlockReasons::Gameplay))
	{
		// This is the first reason added, so we transitioned unmuted -> muted. Tell the other PC to mute this one.
		OwningPC->ClientUnmutePlayer(UnmuteId);
	}
}

void FPlayerMuteList::GameplayUnmuteAllPlayers(APlayerController* OwningPC)
{
	TArray<FUniqueNetIdRepl> PlayersToUnmute;

	for (TTuple<FUniqueNetIdRef, EVoiceBlockReasons>& PacketFilterEntry : VoicePacketFilterMap)
	{
		if (RemoveVoiceBlockReason(PacketFilterEntry.Key, EVoiceBlockReasons::Gameplay))
		{
			// If there's no reason left to block comms, add it to the array of ids to unmute.
			PlayersToUnmute.Add(PacketFilterEntry.Key);
		}
	}

	if (PlayersToUnmute.Num() > 0)
	{
		// Now process all unmutes on the client
		OwningPC->ClientUnmutePlayers(PlayersToUnmute);
	}
}

void FPlayerMuteList::ServerBlockPlayer(APlayerController* OwningPC, const FUniqueNetIdRepl& BlockId)
{
	// Add block to the list of reasons to block comms.
	if (AddVoiceBlockReason(BlockId.GetUniqueNetId(), EVoiceBlockReasons::Blocked))
	{
		// This is the first reason added, so we transitioned unmuted -> muted. Replicate mute state to client
		OwningPC->ClientMutePlayer(BlockId);
	}

	// Find the muted player's player controller so it can be notified
	APlayerController* OtherPC = OwningPC->GetPlayerControllerForMuting(BlockId);
	if (OtherPC != NULL)
	{
		// Update their packet filter list too
		// But don't inform the other client that they have been blocked
		OtherPC->MuteList.AddVoiceBlockReason(OwningPC->PlayerState->GetUniqueId().GetUniqueNetId(), EVoiceBlockReasons::BlockedBy);
	}
}

void FPlayerMuteList::ServerUnblockPlayer(APlayerController* OwningPC, const FUniqueNetIdRepl& UnblockId)
{
	// Remove gameplay from the list of reasons to block comms.
	if (RemoveVoiceBlockReason(UnblockId.GetUniqueNetId(), EVoiceBlockReasons::Blocked))
	{
		// We removed the last flag, transitioning from muted -> unmuted. Replicate mute state to client.
		OwningPC->ClientUnmutePlayer(UnblockId);
	}

	// Find the muted player's player controller so it can be notified
	APlayerController* OtherPC = OwningPC->GetPlayerControllerForMuting(UnblockId);
	if (OtherPC != NULL)
	{
		// Update their packet filter list too
		OtherPC->MuteList.RemoveVoiceBlockReason(OwningPC->PlayerState->GetUniqueId().GetUniqueNetId(), EVoiceBlockReasons::BlockedBy);
	}
}

bool FPlayerMuteList::IsPlayerMuted(const FUniqueNetId& PlayerId)
{
	const EVoiceBlockReasons* VoiceBlockReasonsForPlayer = VoicePacketFilterMap.Find(PlayerId.AsShared());

	return VoiceBlockReasonsForPlayer && *VoiceBlockReasonsForPlayer != EVoiceBlockReasons::None;
}

FString DumpMutelistState(UWorld* World)
{
	FString Output = "Muting state\n";

	if (World)
	{
		for(FConstPlayerControllerIterator Iterator = World->GetPlayerControllerIterator(); Iterator; ++Iterator)
		{
			if (APlayerController* PlayerController = Iterator->Get())
			{
				Output += FString::Printf(TEXT("Player: %s\n"), PlayerController->PlayerState ? *PlayerController->PlayerState->GetPlayerName() : TEXT("NONAME"));
				Output += FString::Printf(TEXT("VoiceChannel: %d\n"), PlayerController->MuteList.VoiceChannelIdx);
				Output += FString::Printf(TEXT("Handshake: %s\n"), PlayerController->MuteList.bHasVoiceHandshakeCompleted ? TEXT("true") : TEXT("false"));

				Output += FString("System mutes:\n");
				for (const TPair<FUniqueNetIdRef, EVoiceBlockReasons>& PlayerFilterEntry : PlayerController->MuteList.VoicePacketFilterMap)
				{
					if (PlayerFilterEntry.Value != EVoiceBlockReasons::None)
					{
						Output += FString::Printf(TEXT("%s:%s\n"), *PlayerFilterEntry.Key->ToString(), *LexToString(PlayerFilterEntry.Value));
					}
				}

				Output += TEXT("\n");
			}
		}
	}

	return Output;
}

