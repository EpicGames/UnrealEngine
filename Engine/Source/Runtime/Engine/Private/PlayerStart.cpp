// Copyright Epic Games, Inc. All Rights Reserved.

#include "GameFramework/PlayerStart.h"
#include "Async/TaskGraphInterfaces.h"
#include "UObject/ConstructorHelpers.h"
#include "Components/ArrowComponent.h"
#include "Engine/Texture2D.h"
#include "Components/CapsuleComponent.h"
#include "Components/BillboardComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PlayerStart)

APlayerStart::APlayerStart(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	GetCapsuleComponent()->InitCapsuleSize(40.0f, 92.0f);
	GetCapsuleComponent()->SetShouldUpdatePhysicsVolume(false);

#if WITH_EDITORONLY_DATA
	ArrowComponent = CreateEditorOnlyDefaultSubobject<UArrowComponent>(TEXT("Arrow"));

	if (!IsRunningCommandlet())
	{
		// Structure to hold one-time initialization
		struct FConstructorStatics
		{
			ConstructorHelpers::FObjectFinderOptional<UTexture2D> PlayerStartTextureObject;
			FName ID_PlayerStart;
			FText NAME_PlayerStart;
			FName ID_Navigation;
			FText NAME_Navigation;
			FConstructorStatics()
				: PlayerStartTextureObject(TEXT("/Engine/EditorResources/S_Player"))
				, ID_PlayerStart(TEXT("PlayerStart"))
				, NAME_PlayerStart(NSLOCTEXT("SpriteCategory", "PlayerStart", "Player Start"))
				, ID_Navigation(TEXT("Navigation"))
				, NAME_Navigation(NSLOCTEXT("SpriteCategory", "Navigation", "Navigation"))
			{
			}
		};
		static FConstructorStatics ConstructorStatics;

		if (GetGoodSprite())
		{
			GetGoodSprite()->Sprite = ConstructorStatics.PlayerStartTextureObject.Get();
			GetGoodSprite()->SetRelativeScale3D(FVector(0.5f, 0.5f, 0.5f));
			GetGoodSprite()->SpriteInfo.Category = ConstructorStatics.ID_PlayerStart;
			GetGoodSprite()->SpriteInfo.DisplayName = ConstructorStatics.NAME_PlayerStart;
		}
		if (GetBadSprite())
		{
			GetBadSprite()->SetVisibility(false);
		}

		if (ArrowComponent)
		{
			ArrowComponent->ArrowColor = FColor(150, 200, 255);

			ArrowComponent->ArrowSize = 1.0f;
			ArrowComponent->bTreatAsASprite = true;
			ArrowComponent->SpriteInfo.Category = ConstructorStatics.ID_Navigation;
			ArrowComponent->SpriteInfo.DisplayName = ConstructorStatics.NAME_Navigation;
			ArrowComponent->SetupAttachment(GetCapsuleComponent());
			ArrowComponent->bIsScreenSizeScaled = true;
		}
	}

	bIsSpatiallyLoaded = false;
#endif // WITH_EDITORONLY_DATA
}

#if WITH_EDITORONLY_DATA
/** Returns ArrowComponent subobject **/
UArrowComponent* APlayerStart::GetArrowComponent() const { return ArrowComponent; }
#endif

