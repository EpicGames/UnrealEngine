// Copyright Epic Games, Inc. All Rights Reserved.

#include "GameFramework/PlayerState.h"
#include "Engine/World.h"
#include "GameFramework/EngineMessage.h"
#include "Net/UnrealNetwork.h"
#include "Net/OnlineEngineInterface.h"
#include "GameFramework/GameStateBase.h"
#include "Net/Core/PushModel/PushModel.h"
#include "Net/Iris/ReplicationSystem/ReplicationSystemUtil.h"
#include "Iris/ReplicationSystem/ReplicationSystem.h"
#include "Iris/ReplicationSystem/Prioritization/NetObjectPrioritizer.h"

#if UE_WITH_REMOTE_OBJECT_HANDLE
#include "UObject/RemoteObject.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(PlayerState)

APlayerState::APlayerState(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer
		.DoNotCreateDefaultSubobject(TEXT("Sprite")) )
{
	SetRemoteRoleForBackwardsCompat(ROLE_SimulatedProxy);
	bReplicates = true;
	bAlwaysRelevant = true;
	SetReplicatingMovement(false);
	SetNetUpdateFrequency(1);

	// Note: this is very important to set to false. Though all replication infos are spawned at run time, during seamless travel
	// they are held on to and brought over into the new world. In ULevel::InitializeActors, these PlayerStates may be treated as map/startup actors
	// and given static NetGUIDs. This also causes their deletions to be recorded and sent to new clients, which if unlucky due to name conflicts,
	// may end up deleting the new PlayerStates they had just spaned.
	bNetLoadOnClient = false;

	EngineMessageClass = UEngineMessage::StaticClass();
	SessionName = NAME_GameSession;

	bShouldUpdateReplicatedPing = true; // Preserved behavior before bShouldUpdateReplicatedPing was added
	bUseCustomPlayerNames = false;
}

void APlayerState::UpdatePing(float InPing)
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_PlayerState_UpdatePing);

	// Limit the size of the ping, to avoid overflowing PingBucket values
	InPing = FMath::Min(1.1f, InPing);

	float CurTime = GetWorld()->RealTimeSeconds;

	float InPingInMs = InPing * 1000.f;

	if ((CurTime - CurPingBucketTimestamp) >= 1.f)
	{
		// Trigger ping recalculation now, while all buckets are 'full'
		//	(misses the latest ping update, but averages a full 4 seconds data)
		RecalculateAvgPing();

		CurPingBucket = (CurPingBucket + 1) % UE_ARRAY_COUNT(PingBucket);
		CurPingBucketTimestamp = CurTime;


		PingBucket[CurPingBucket].PingSum = FMath::FloorToInt(InPingInMs);
		PingBucket[CurPingBucket].PingCount = 1;
	}
	// Limit the number of pings we accept per-bucket, to avoid overflowing PingBucket values
	else if (PingBucket[CurPingBucket].PingCount < 7)
	{
		PingBucket[CurPingBucket].PingSum += FMath::FloorToInt(InPingInMs);
		PingBucket[CurPingBucket].PingCount++;
	}
}

void APlayerState::RecalculateAvgPing()
{
	int32 Sum = 0;
	int32 Count = 0;

	for (uint8 i=0; i<UE_ARRAY_COUNT(PingBucket); i++)
	{
		Sum += PingBucket[i].PingSum;
		Count += PingBucket[i].PingCount;
	}

	// Calculate the average, and divide it by 4 to optimize replication
	ExactPing = (Count > 0 ? ((float)Sum / (float)Count) : 0.f);

	if (bShouldUpdateReplicatedPing || !HasAuthority())
	{
		SetCompressedPing(FMath::Min(255, (int32)(ExactPing * 0.25f)));
	}
}

void APlayerState::DispatchOverrideWith(APlayerState* PlayerState)
{
	OverrideWith(PlayerState);
	ReceiveOverrideWith(PlayerState);
}

void APlayerState::DispatchCopyProperties(APlayerState* PlayerState)
{
	CopyProperties(PlayerState);
	ReceiveCopyProperties(PlayerState);
}

void APlayerState::OverrideWith(APlayerState* PlayerState)
{
	SetIsSpectator(PlayerState->IsSpectator());
	SetIsOnlyASpectator(PlayerState->IsOnlyASpectator());
	SetUniqueId(PlayerState->GetUniqueId());
	SetPlayerNameInternal(PlayerState->GetPlayerName());
}


void APlayerState::CopyProperties(APlayerState* PlayerState)
{
	PlayerState->SetScore(GetScore());
	PlayerState->SetCompressedPing(GetCompressedPing());
	PlayerState->ExactPing = ExactPing;
	PlayerState->SetPlayerId(GetPlayerId());
	PlayerState->SetUniqueId(GetUniqueId());
	PlayerState->SetPlayerNameInternal(GetPlayerName());
	PlayerState->SetStartTime(GetStartTime());
	PlayerState->SavedNetworkAddress = SavedNetworkAddress;
}

void APlayerState::OnDeactivated()
{
	// By default we duplicate the inactive player state and destroy the old one
	Destroy();
}

void APlayerState::OnReactivated()
{
	// Stub
}

void APlayerState::PostInitializeComponents()
{
	Super::PostInitializeComponents();

	UWorld* World = GetWorld();
	AGameStateBase* GameStateBase = World->GetGameState();

	// register this PlayerState with the game state
	if (GameStateBase != nullptr )
	{
		GameStateBase->AddPlayerState(this);
	}

	if (GetLocalRole() < ROLE_Authority)
	{
		return;
	}

	AController* OwningController = GetOwningController();
	if (OwningController != nullptr)
	{
		SetIsABot(Cast<APlayerController>(OwningController) == nullptr);
	}

	if (GameStateBase)
	{
		SetStartTime(GameStateBase->GetPlayerStartTime(OwningController));
	}
}

class AController* APlayerState::GetOwningController() const
{
	return Cast<AController>(GetOwner());
}

class APlayerController* APlayerState::GetPlayerController() const
{
	return Cast<APlayerController>(GetOwner());
}

void APlayerState::ClientInitialize(AController* C)
{
	SetOwner(C);
}

void APlayerState::OnRep_Score()
{
}

void APlayerState::OnRep_bIsInactive()
{
	// remove and re-add from the GameState so it's in the right list  
	UWorld* World = GetWorld();
	if (World && World->GetGameState())
	{
		World->GetGameState()->RemovePlayerState(this);
		World->GetGameState()->AddPlayerState(this);
	}
}

bool APlayerState::ShouldBroadCastWelcomeMessage(bool bExiting)
{
	return (!IsInactive() && GetNetMode() != NM_Standalone);
}

void APlayerState::Destroyed()
{
	UWorld* World = GetWorld();
	if (World->GetGameState() != nullptr)
	{
		World->GetGameState()->RemovePlayerState(this);
	}

	if( ShouldBroadCastWelcomeMessage(true) )
	{
		for (FConstPlayerControllerIterator Iterator = World->GetPlayerControllerIterator(); Iterator; ++Iterator)
		{
			APlayerController* PlayerController = Iterator->Get();
			if( PlayerController )
			{
				PlayerController->ClientReceiveLocalizedMessage( EngineMessageClass, 4, this);
			}
		}
	}

	// Remove the player from the online session
	UnregisterPlayerWithSession();
	Super::Destroyed();
}


void APlayerState::Reset()
{
	Super::Reset();
	SetScore(0);
	ForceNetUpdate();
}

#if UE_WITH_REMOTE_OBJECT_HANDLE
FRemoteServerId APlayerState::GetPreferredRemoteServer() const
{
	if (!AutoRTFM::IsClosed() && PawnPrivate.GetResidence() == EResidence::Remote)
	{
		// We are not in a transaction (probably during push migration) and our Pawn is non-local
		// So the Pawn must have already migrated, choose that server.
		FRemoteServerId PawnServerId = UE::RemoteObject::Handle::GetOwnerServerId(PawnPrivate.GetRemoteId());
		if (PawnServerId.IsValid())
		{
			return PawnServerId;
		}
	}
	else if (APawn* LocalPawn = PawnPrivate.Get())
	{
		return LocalPawn->GetPreferredRemoteServer();
	}
	// No Pawn — stay where we are (don't use spatial delegate, we have no location)
	return UE::RemoteObject::Handle::GetOwnerServerId(this);
}
#endif

FString APlayerState::GetHumanReadableName() const
{
	return GetPlayerName();
}

void APlayerState::OnRep_PlayerName()
{
	OldNamePrivate = GetPlayerName();
	HandleWelcomeMessage();
}

void APlayerState::SetPlayerNameInternal(const FString& S)
{
	MARK_PROPERTY_DIRTY_FROM_NAME(APlayerState, PlayerNamePrivate, this);
	PlayerNamePrivate = S;
}

void APlayerState::SetPlayerName(const FString& S)
{
	SetPlayerNameInternal(S);

	// RepNotify callback won't get called by net code if we are the server
	ENetMode NetMode = GetNetMode();
	if (NetMode == NM_Standalone || NetMode == NM_ListenServer)
	{
		OnRep_PlayerName();
	}

	OldNamePrivate = GetPlayerName();
	ForceNetUpdate();
}

FString APlayerState::GetPlayerName() const
{
	return bUseCustomPlayerNames ? GetPlayerNameCustom() : PlayerNamePrivate;
}

FString APlayerState::GetPlayerNameCustom() const
{
	return PlayerNamePrivate;
}

FString APlayerState::GetOldPlayerName() const
{
	return OldNamePrivate;
}

void APlayerState::SetOldPlayerName(const FString& S)
{
	OldNamePrivate = S;
}

void APlayerState::HandleWelcomeMessage()
{
	UWorld* World = GetWorld();
	if (World == nullptr || World->TimeSeconds < 2)
	{
		bHasBeenWelcomed = true;
		return;
	}

	// new player or name change
	if (bHasBeenWelcomed)
	{
		if (ShouldBroadCastWelcomeMessage())
		{
			for (FConstPlayerControllerIterator Iterator = World->GetPlayerControllerIterator(); Iterator; ++Iterator)
			{
				APlayerController* PlayerController = Iterator->Get();
				if (PlayerController)
				{
					PlayerController->ClientReceiveLocalizedMessage(EngineMessageClass, 2, this);
				}
			}
		}
	}
	else
	{
		int32 WelcomeMessageNum = IsOnlyASpectator() ? 16 : 1;
		bHasBeenWelcomed = true;

		if (ShouldBroadCastWelcomeMessage())
		{
			for (FConstPlayerControllerIterator Iterator = World->GetPlayerControllerIterator(); Iterator; ++Iterator)
			{
				APlayerController* PlayerController = Iterator->Get();
				if (PlayerController)
				{
					PlayerController->ClientReceiveLocalizedMessage(EngineMessageClass, WelcomeMessageNum, this);
				}
			}
		}
	}
}

void APlayerState::OnRep_PlayerId()
{
}

void APlayerState::OnRep_UniqueId()
{
	// First notify it's changed
	OnSetUniqueId();

	// Register player with session
	RegisterPlayerWithSession(false);
}

void APlayerState::RegisterPlayerWithSession(bool bWasFromInvite)
{
	if (GetNetMode() != NM_Standalone)
	{
		if (GetUniqueId().IsValid()) // May not be valid if this is was created via DebugCreatePlayer
		{
			// Register the player as part of the session
			const APlayerState* PlayerState = GetDefault<APlayerState>();
			if (UOnlineEngineInterface::Get()->DoesSessionExist(GetWorld(), PlayerState->SessionName))
			{
				UOnlineEngineInterface::Get()->RegisterPlayer(GetWorld(), PlayerState->SessionName, GetUniqueId(), bWasFromInvite);
			}
		}
	}
}

void APlayerState::UnregisterPlayerWithSession()
{
	if (GetNetMode() == NM_Client && GetUniqueId().IsValid())
	{
		const APlayerState* PlayerState = GetDefault<APlayerState>();
		if (PlayerState->SessionName != NAME_None)
		{
			if (UOnlineEngineInterface::Get()->DoesSessionExist(GetWorld(), PlayerState->SessionName))
			{
				UOnlineEngineInterface::Get()->UnregisterPlayer(GetWorld(), PlayerState->SessionName, GetUniqueId());
			}
		}
	}
}

APlayerState* APlayerState::Duplicate()
{
	FActorSpawnParameters SpawnInfo;
	SpawnInfo.Instigator = GetInstigator();
	SpawnInfo.SpawnCollisionHandlingOverride = ESpawnActorCollisionHandlingMethod::AlwaysSpawn;
	SpawnInfo.ObjectFlags |= RF_Transient;	// We never want to save player states into a map
	APlayerState* NewPlayerState = GetWorld()->SpawnActor<APlayerState>(GetClass(), SpawnInfo );
	// Can fail in case of multiplayer PIE teardown
	if (NewPlayerState)
	{
		DispatchCopyProperties(NewPlayerState);
	}
	return NewPlayerState;
}

void APlayerState::SeamlessTravelTo(APlayerState* NewPlayerState)
{
	// By default there should always be a player state by the time it calls SeamlessTravelTo, but this can be overridden in subclasses
	if (ensure(NewPlayerState))
	{
		DispatchCopyProperties(NewPlayerState);
		NewPlayerState->SetIsOnlyASpectator(IsOnlyASpectator());
	}
}

void APlayerState::GetLifetimeReplicatedProps(TArray< FLifetimeProperty > & OutLifetimeProps) const
{
	Super::GetLifetimeReplicatedProps(OutLifetimeProps);

	FDoRepLifetimeParams SharedParams;
	SharedParams.bIsPushBased = true;

	DOREPLIFETIME_WITH_PARAMS_FAST(APlayerState, Score, SharedParams);
	DOREPLIFETIME_WITH_PARAMS_FAST(APlayerState, bIsSpectator, SharedParams);
	DOREPLIFETIME_WITH_PARAMS_FAST(APlayerState, bOnlySpectator, SharedParams);
	DOREPLIFETIME_WITH_PARAMS_FAST(APlayerState, bFromPreviousLevel, SharedParams);
	DOREPLIFETIME_WITH_PARAMS_FAST(APlayerState, StartTime, SharedParams);
	DOREPLIFETIME_WITH_PARAMS_FAST(APlayerState, PlayerNamePrivate, SharedParams);

	SharedParams.Condition = COND_SkipOwner;
	DOREPLIFETIME_WITH_PARAMS_FAST(APlayerState, CompressedPing, SharedParams);

	SharedParams.Condition = COND_InitialOnly;
	DOREPLIFETIME_WITH_PARAMS_FAST(APlayerState, PlayerId, SharedParams);
	DOREPLIFETIME_WITH_PARAMS_FAST(APlayerState, bIsABot, SharedParams);
	DOREPLIFETIME_WITH_PARAMS_FAST(APlayerState, bIsInactive, SharedParams);
	DOREPLIFETIME_WITH_PARAMS_FAST(APlayerState, UniqueId, SharedParams);
}


void APlayerState::SetScore(const float NewScore)
{
	MARK_PROPERTY_DIRTY_FROM_NAME(APlayerState, Score, this);
	Score = NewScore;
}

void APlayerState::SetPlayerId(const int32 NewId)
{
	MARK_PROPERTY_DIRTY_FROM_NAME(APlayerState, PlayerId, this);
	PlayerId = NewId;
}

void APlayerState::SetCompressedPing(const uint8 NewPing)
{
	MARK_PROPERTY_DIRTY_FROM_NAME(APlayerState, CompressedPing, this);
	CompressedPing = NewPing;
}

float APlayerState::GetPingInMilliseconds() const
{
	if (ExactPing > 0.0f)
	{
		// Prefer the exact ping if set (only on the server or for the local players)
		return ExactPing;
	}
	else
	{
		// Otherwise, use the replicated compressed ping
		return CompressedPing * 4.0f;
	}
}

void APlayerState::SetIsSpectator(const bool bNewSpectator)
{
	MARK_PROPERTY_DIRTY_FROM_NAME(APlayerState, bIsSpectator, this);
	bIsSpectator = bNewSpectator;
}

void APlayerState::SetIsOnlyASpectator(const bool bNewSpectator)
{
	MARK_PROPERTY_DIRTY_FROM_NAME(APlayerState, bOnlySpectator, this);
	bOnlySpectator = bNewSpectator;
}

void APlayerState::SetIsABot(const bool bNewIsABot)
{
	MARK_PROPERTY_DIRTY_FROM_NAME(APlayerState, bIsABot, this);
	bIsABot = bNewIsABot;
}

void APlayerState::SetIsInactive(const bool bNewInactive)
{
	MARK_PROPERTY_DIRTY_FROM_NAME(APlayerState, bIsInactive, this);
	bIsInactive = bNewInactive;
}

void APlayerState::SetIsFromPreviousLevel(const bool bNewFromPreviousLevel)
{
	MARK_PROPERTY_DIRTY_FROM_NAME(APlayerState, bFromPreviousLevel, this);
	bFromPreviousLevel = bNewFromPreviousLevel;
}

void APlayerState::SetStartTime(const int32 NewStartTime)
{
	MARK_PROPERTY_DIRTY_FROM_NAME(APlayerState, StartTime, this);
	StartTime = NewStartTime;
}

FUniqueNetIdRepl APlayerState::BP_GetUniqueId() const
{
	return GetUniqueId();
}

void APlayerState::SetUniqueId(const FUniqueNetIdRepl& NewUniqueId)
{
	MARK_PROPERTY_DIRTY_FROM_NAME(APlayerState, UniqueId, this);
	UniqueId = NewUniqueId;
	OnSetUniqueId();
}

void APlayerState::SetUniqueId(FUniqueNetIdRepl&& NewUniqueId)
{
	MARK_PROPERTY_DIRTY_FROM_NAME(APlayerState, UniqueId, this);
	UniqueId = MoveTemp(NewUniqueId);
	OnSetUniqueId();
}

void APlayerState::OnSetUniqueId()
{

}

void APlayerState::SetPawnPrivate(APawn* InPawn)
{
	if (InPawn != PawnPrivate)
	{
		if (PawnPrivate)
		{
			PawnPrivate->OnDestroyed.RemoveDynamic(this, &APlayerState::OnPawnPrivateDestroyed);
		}
		PawnPrivate = InPawn;
		if (PawnPrivate)
		{
			PawnPrivate->OnDestroyed.AddDynamic(this, &APlayerState::OnPawnPrivateDestroyed);
		}
	}
}

void APlayerState::OnPawnPrivateDestroyed(AActor* InActor)
{
	if (InActor == PawnPrivate)
	{
		PawnPrivate = nullptr;
	}
}

