// Copyright Epic Games, Inc. All Rights Reserved.

#include "PluginBlueprintLibrary.h"

#include "Containers/Array.h"
#include "Containers/Set.h"
#include "Containers/UnrealString.h"
#include "Interfaces/IPluginManager.h"
#include "Interfaces/IProjectManager.h"
#include "Misc/PackageName.h"
#include "Templates/SharedPointer.h"
#include "UObject/SoftObjectPath.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PluginBlueprintLibrary)


TArray<FString> UPluginBlueprintLibrary::GetAdditionalPluginSearchPaths()
{
	TArray<FString> PluginSearchPaths;

	const TSet<FString>& SearchPathsSet =
		IPluginManager::Get().GetAdditionalPluginSearchPaths();
	for (const FString& SearchPath : SearchPathsSet)
	{
		PluginSearchPaths.Add(SearchPath);
	}

	return PluginSearchPaths;
}

const TArray<FString>&
UPluginBlueprintLibrary::GetAdditionalProjectPluginSearchPaths()
{
	return IProjectManager::Get().GetAdditionalPluginDirectories();
}

TArray<FString> UPluginBlueprintLibrary::GetEnabledPluginNames()
{
	const TArray<TSharedRef<IPlugin>> EnabledPlugins =
		IPluginManager::Get().GetEnabledPlugins();

	TArray<FString> PluginNames;
	PluginNames.Reserve(EnabledPlugins.Num());

	for (const TSharedRef<IPlugin>& Plugin : EnabledPlugins)
	{
		PluginNames.Add(Plugin->GetName());
	}

	return PluginNames;
}

bool UPluginBlueprintLibrary::GetPluginNameForObjectPath(
		const FSoftObjectPath& ObjectPath,
		FString& OutPluginName)
{
	const TSharedPtr<IPlugin> Plugin =
		IPluginManager::Get().FindPluginFromPath(
			ObjectPath.GetAssetPathString());
	if (!Plugin)
	{
		return false;
	}

	OutPluginName = Plugin->GetName();

	return true;
}

bool UPluginBlueprintLibrary::GetPluginDescriptorFilePath(
		const FString& PluginName,
		FString& OutFilePath)
{
	const TSharedPtr<IPlugin> Plugin =
		IPluginManager::Get().FindPlugin(PluginName);
	if (!Plugin)
	{
		return false;
	}

	OutFilePath = Plugin->GetDescriptorFileName();

	return true;
}

bool UPluginBlueprintLibrary::GetPluginBaseDir(
		const FString& PluginName,
		FString& OutBaseDir)
{
	const TSharedPtr<IPlugin> Plugin =
		IPluginManager::Get().FindPlugin(PluginName);
	if (!Plugin)
	{
		return false;
	}

	OutBaseDir = Plugin->GetBaseDir();

	return true;
}

bool UPluginBlueprintLibrary::GetPluginContentDir(
		const FString& PluginName,
		FString& OutContentDir)
{
	const TSharedPtr<IPlugin> Plugin =
		IPluginManager::Get().FindPlugin(PluginName);
	if (!Plugin)
	{
		return false;
	}

	OutContentDir = Plugin->GetContentDir();

	return true;
}

bool UPluginBlueprintLibrary::GetPluginMountedAssetPath(
		const FString& PluginName,
		FString& OutAssetPath)
{
	const TSharedPtr<IPlugin> Plugin =
		IPluginManager::Get().FindPlugin(PluginName);
	if (!Plugin)
	{
		return false;
	}

	OutAssetPath = Plugin->GetMountedAssetPath();

	return true;
}

bool UPluginBlueprintLibrary::GetPluginVersion(
		const FString& PluginName,
		int32& OutVersion)
{
	const TSharedPtr<IPlugin> Plugin =
		IPluginManager::Get().FindPlugin(PluginName);
	if (!Plugin)
	{
		return false;
	}

	OutVersion = Plugin->GetDescriptor().Version;

	return true;
}

bool UPluginBlueprintLibrary::GetPluginVersionName(
		const FString& PluginName,
		FString& OutVersionName)
{
	const TSharedPtr<IPlugin> Plugin =
		IPluginManager::Get().FindPlugin(PluginName);
	if (!Plugin)
	{
		return false;
	}

	OutVersionName = Plugin->GetDescriptor().VersionName;

	return true;
}

bool UPluginBlueprintLibrary::GetPluginDescription(
		const FString& PluginName,
		FString& OutDescription)
{
	const TSharedPtr<IPlugin> Plugin =
		IPluginManager::Get().FindPlugin(PluginName);
	if (!Plugin)
	{
		return false;
	}

	OutDescription = Plugin->GetDescriptor().Description;

	return true;
}

bool UPluginBlueprintLibrary::GetPluginEditorCustomVirtualPath(
		const FString& PluginName,
		FString& OutVirtualPath)
{
	const TSharedPtr<IPlugin> Plugin =
		IPluginManager::Get().FindPlugin(PluginName);
	if (!Plugin)
	{
		return false;
	}

	OutVirtualPath = Plugin->GetDescriptor().EditorCustomVirtualPath;

	return true;
}

bool UPluginBlueprintLibrary::IsPluginMounted(const FString& PluginName)
{
	FString PluginMountedAssetPath;
	if (!GetPluginMountedAssetPath(PluginName, PluginMountedAssetPath))
	{
		return false;
	}

	return FPackageName::MountPointExists(PluginMountedAssetPath);
}
