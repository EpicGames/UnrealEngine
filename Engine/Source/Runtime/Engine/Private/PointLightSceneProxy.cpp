// Copyright Epic Games, Inc. All Rights Reserved.

#include "PointLightSceneProxy.h"
#include "PointLightSceneProxyDesc.h"
#include "Components/PointLightComponent.h"
#include "SceneInterface.h"
#include "SceneManagement.h"
#include "SceneView.h"
#include "RenderUtils.h"

int32 GAllowPointLightCubemapShadows = 1;
static FAutoConsoleVariableRef CVarAllowPointLightCubemapShadows(
	TEXT("r.AllowPointLightCubemapShadows"),
	GAllowPointLightCubemapShadows,
	TEXT("When 0, will prevent point light cube map shadows from being used and the light will be unshadowed.")
);

FPointLightSceneProxy::FPointLightSceneProxy(const UPointLightComponent* Component)
	: FPointLightSceneProxy(UPointLightComponent::BuildSceneProxyDesc(*Component))
{
}

FPointLightSceneProxy::FPointLightSceneProxy(const FPointLightSceneProxyDesc& LightDesc)
	: FLocalLightSceneProxy(LightDesc)
	, FalloffExponent(LightDesc.LightFalloffExponent)
	, SourceRadius(LightDesc.SourceRadius)
	, SoftSourceRadius(LightDesc.SoftSourceRadius)
	, SourceLength(LightDesc.SourceLength)
	, bInverseSquared(LightDesc.bUseInverseSquaredFalloff)
{
}

float FPointLightSceneProxy::GetSourceRadius() const 
{
	return SourceRadius;
}

bool FPointLightSceneProxy::IsInverseSquared() const 
{
	return bInverseSquared;
}

/** Accesses parameters needed for rendering the light. */
void FPointLightSceneProxy::GetLightShaderParameters(FLightRenderParameters& LightParameters, uint32 Flags) const
{
	LightParameters.WorldPosition = GetOrigin();
	LightParameters.InvRadius = InvRadius;
	LightParameters.Color = GetColor();
	LightParameters.FalloffExponent = bInverseSquared ? 0.0f : FalloffExponent;

	// TODO LWC - GetDirection() seems like it needs to be normalized, somehow accumulating error with large-scale position values
	LightParameters.Direction = (FVector3f)-GetDirection(); // LWC_TODO: Precision Loss
	LightParameters.Tangent = FVector3f(WorldToLight.M[0][2], WorldToLight.M[1][2], WorldToLight.M[2][2]);
	LightParameters.SpotAngles = FVector2f(-2.0f, 1.0f);
	LightParameters.SpecularScale = FMath::Clamp(SpecularScale, 0.f, 1.f);
	LightParameters.DiffuseScale = FMath::Clamp(DiffuseScale, 0.f, 1.f);
	LightParameters.SourceRadius = SourceRadius;
	LightParameters.SoftSourceRadius = SoftSourceRadius;
	LightParameters.SourceLength = SourceLength;
	LightParameters.RectLightBarnCosAngle = 0.0f;
	LightParameters.RectLightBarnLength = -2.0f;
	LightParameters.RectLightAtlasUVOffset = FVector2f::ZeroVector;
	LightParameters.RectLightAtlasUVScale = FVector2f::ZeroVector;
	LightParameters.RectLightAtlasMaxLevel = FLightRenderParameters::GetRectLightAtlasInvalidMIPLevel();
	LightParameters.IESAtlasIndex = INDEX_NONE;
	LightParameters.InverseExposureBlend = InverseExposureBlend;
	LightParameters.LightFunctionAtlasLightIndex = GetLightFunctionAtlasLightIndex();
	LightParameters.bAffectsTranslucentLighting = AffectsTranslucentLighting() ? 1 : 0;

	if (IESAtlasId != ~0)
	{
		GetSceneInterface()->GetLightIESAtlasSlot(this, &LightParameters);
	}
}

/**
* Sets up a projected shadow initializer for shadows from the entire scene.
* @return True if the whole-scene projected shadow should be used.
*/
bool FPointLightSceneProxy::GetWholeSceneProjectedShadowInitializer(const FSceneViewFamily& ViewFamily, TArray<FWholeSceneProjectedShadowInitializer, TInlineAllocator<6> >& OutInitializers) const
{
	if ((ViewFamily.GetFeatureLevel() >= ERHIFeatureLevel::SM5 || DoesRuntimeSupportOnePassPointLightShadows(ViewFamily.GetShaderPlatform()))
		&& GAllowPointLightCubemapShadows != 0)
	{
		FWholeSceneProjectedShadowInitializer& OutInitializer = *new(OutInitializers) FWholeSceneProjectedShadowInitializer;
		OutInitializer.PreShadowTranslation = -GetLightToWorld().GetOrigin();
		OutInitializer.WorldToLight = GetWorldToLight().RemoveTranslation();
		OutInitializer.Scales = FVector2D(1, 1);
		OutInitializer.SubjectBounds = FBoxSphereBounds(FVector(0, 0, 0), FVector(Radius, Radius, Radius), Radius);
		OutInitializer.WAxis = FVector4(0, 0, 1, 0);
		OutInitializer.MinLightW = 0.1f;
		OutInitializer.MaxDistanceToCastInLightW = Radius;
		OutInitializer.bOnePassPointLightShadow = true;
		OutInitializer.bRayTracedDistanceField = UseRayTracedDistanceFieldShadows() && DoesPlatformSupportDistanceFieldShadowing(ViewFamily.GetShaderPlatform());
		return true;
	}

	return false;
}

FVector FPointLightSceneProxy::GetPerObjectProjectedShadowProjectionPoint(const FBoxSphereBounds& SubjectBounds) const
{
	return FMath::ClosestPointOnSegment(SubjectBounds.Origin, GetOrigin() - GetDirection() * SourceLength * 0.5f, GetOrigin() + GetDirection() * SourceLength * 0.5f);
}