// Copyright Epic Games, Inc. All Rights Reserved.

#include "PointLightSceneProxyDesc.h"

void FPointLightSceneProxyDesc::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

	Ar << LightFalloffExponent;
	Ar << SourceRadius;
	Ar << SoftSourceRadius;
	Ar << SourceLength;
	Ar << bUseInverseSquaredFalloff;
}