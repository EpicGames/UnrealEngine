// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	Polygon.cpp: FPoly implementation (Editor polygons).
=============================================================================*/

#include "Math/RotationMatrix.h"
#include "Model.h"
#include "Engine/Brush.h"

#include "GeomTools.h"
#include "Engine/Polys.h"


DEFINE_LOG_CATEGORY_STATIC(LogPolygon, Log, All);

#define PolyCheck( x )			check( x )

/*-----------------------------------------------------------------------------
	FLightmassPrimitiveSettings
-----------------------------------------------------------------------------*/
// Functions.
FArchive& operator<<(FArchive& Ar, FLightmassPrimitiveSettings& Settings)
{
	bool Temp = Settings.bUseTwoSidedLighting;
	Ar << Temp;
	Settings.bUseTwoSidedLighting = Temp;

	Temp = Settings.bShadowIndirectOnly;
	Ar << Temp;
	Settings.bShadowIndirectOnly = Temp;

	Ar << Settings.FullyOccludedSamplesFraction;

	Temp = Settings.bUseEmissiveForStaticLighting;
	Ar << Temp;
	Settings.bUseEmissiveForStaticLighting = Temp;

	if (Ar.UEVer() >= VER_UE4_NEW_LIGHTMASS_PRIMITIVE_SETTING)
	{
		Temp = Settings.bUseVertexNormalForHemisphereGather;
		Ar << Temp;
		Settings.bUseVertexNormalForHemisphereGather = Temp;
	}
	
	Ar << Settings.EmissiveLightFalloffExponent;
	Ar << Settings.EmissiveLightExplicitInfluenceRadius;
	
	Ar << Settings.EmissiveBoost;
	Ar << Settings.DiffuseBoost;

	return Ar;
}

/**
 * Constructor, initializing all member variables.
 */
FPoly::FPoly()
	: Base(ForceInitToZero)
	, Normal(ForceInitToZero)
	, TextureU(ForceInitToZero)
	, TextureV(ForceInitToZero)
	, PolyFlags(PF_DefaultFlags)
	, Actor(nullptr)
	, Material(nullptr)
	, iLink(INDEX_NONE)
	, iLinkSurf(INDEX_NONE)
	, iBrushPoly(INDEX_NONE)
	, SmoothingMask(0)
	, LightMapScale(32.0f)
{
}


void FPoly::Init()
{
	Base			= FVector3f::ZeroVector;
	Normal			= FVector3f::ZeroVector;
	TextureU		= FVector3f::ZeroVector;
	TextureV		= FVector3f::ZeroVector;
	Vertices.Empty();
	PolyFlags       = PF_DefaultFlags;
	Actor			= NULL;
	Material        = NULL;
	RulesetVariation= NAME_None;
	ItemName        = NAME_None;
	iLink           = INDEX_NONE;
	iLinkSurf		= INDEX_NONE;
	iBrushPoly		= INDEX_NONE;
	SmoothingMask	= 0;
	LightMapScale	= 32.0f;

	LightmassSettings = FLightmassPrimitiveSettings();
}


void FPoly::Reverse()
{
	FVector3f Temp;
	int32 i,c;

	Normal *= -1;

	c=Vertices.Num()/2;
	for( i=0; i<c; i++ )
	{
		// Flip all points except middle if odd number of points.
		Temp      = Vertices[i];

		Vertices[i] = Vertices[(Vertices.Num()-1)-i];
		Vertices[(Vertices.Num()-1)-i] = Temp;
	}
}


int32 FPoly::Fix()
{
	int32 i,j,prev;

	j=0; prev=Vertices.Num()-1;
	for( i=0; i<Vertices.Num(); i++ )
	{
		if( !FVector3f::PointsAreSame( Vertices[i], Vertices[prev] ) )
		{
			if( j != i )
				Vertices[j] = Vertices[i];
			prev = j;
			j    ++;
		}
// 		else UE_LOGF(LogPolygon, Warning, "FPoly::Fix: Collapsed a point" );
	}
	if(j < 3)
	{
		Vertices.Empty();
	}
	else if(j < Vertices.Num())
	{
		Vertices.RemoveAt(j,Vertices.Num() - j);
	}
	return Vertices.Num();
}


float FPoly::Area()
{
	// If there are less than 3 verts
	if(Vertices.Num() < 3)
	{
		return 0.f;
	}

	FVector3f Side1,Side2;
	float Area;
	int32 i;

	Area  = 0.f;
	Side1 = Vertices[1] - Vertices[0];
	for( i=2; i<Vertices.Num(); i++ )
	{
		Side2 = Vertices[i] - Vertices[0];
		Area += (Side1 ^ Side2).Size() * 0.5f;
		Side1 = Side2;
	}
	return Area;
}


int32 FPoly::SplitWithPlane
(
	const FVector3f	&PlaneBase,
	const FVector3f	&PlaneNormal,
	FPoly			*FrontPoly,
	FPoly			*BackPoly,
	int32				VeryPrecise
) const
{
	FVector 	Intersection;
	float   	Dist=0,MaxDist=0,MinDist=0;
	float		PrevDist,Thresh;
	enum 	  	{V_FRONT,V_BACK,V_EITHER} Status,PrevStatus=V_EITHER;
	int32     	i,j;

	if (VeryPrecise)	Thresh = UE_THRESH_SPLIT_POLY_PRECISELY;	
	else				Thresh = UE_THRESH_SPLIT_POLY_WITH_PLANE;

	// Find number of vertices.
	check(Vertices.Num()>=3);

	// See if the polygon is split by SplitPoly, or it's on either side, or the
	// polys are coplanar.  Go through all of the polygon points and
	// calculate the minimum and maximum signed distance (in the direction
	// of the normal) from each point to the plane of SplitPoly.
	for( i=0; i<Vertices.Num(); i++ )
	{
		Dist = FVector3f::PointPlaneDist( Vertices[i], PlaneBase, PlaneNormal );

		if( i==0 || Dist>MaxDist ) MaxDist=Dist;
		if( i==0 || Dist<MinDist ) MinDist=Dist;

		if      (Dist > +Thresh) PrevStatus = V_FRONT;
		else if (Dist < -Thresh) PrevStatus = V_BACK;
	}
	if( MaxDist<Thresh && MinDist>-Thresh )
	{
		return SP_Coplanar;
	}
	else if( MaxDist < Thresh )
	{
		return SP_Back;
	}
	else if( MinDist > -Thresh )
	{
		return SP_Front;
	}
	else
	{
		// Split.
		if( FrontPoly == nullptr || BackPoly == nullptr )
			return SP_Split; // Caller only wanted status.

		*FrontPoly = *this; // Copy all info.
		FrontPoly->PolyFlags |= PF_EdCut; // Mark as cut.
		FrontPoly->Vertices.Empty();

		*BackPoly = *this; // Copy all info.
		BackPoly->PolyFlags |= PF_EdCut; // Mark as cut.
		BackPoly->Vertices.Empty();

		j = Vertices.Num()-1; // Previous vertex; have PrevStatus already.

		for( i=0; i<Vertices.Num(); i++ )
		{
			PrevDist	= Dist;
	  		Dist		= FVector3f::PointPlaneDist( Vertices[i], PlaneBase, PlaneNormal );

			if      (Dist > +Thresh)  	Status = V_FRONT;
			else if (Dist < -Thresh)  	Status = V_BACK;
			else						Status = PrevStatus;

			if( Status != PrevStatus )
	        {
				// Crossing.  Either Front-to-Back or Back-To-Front.
				// Intersection point is naturally on both front and back polys.
				if( (Dist >= -Thresh) && (Dist < +Thresh) )
				{
					// This point lies on plane.
					if( PrevStatus == V_FRONT )
					{
						new(FrontPoly->Vertices) FVector3f(Vertices[i]);
						new(BackPoly->Vertices) FVector3f(Vertices[i]);
					}
					else
					{
						new(BackPoly->Vertices) FVector3f(Vertices[i]);
						new(FrontPoly->Vertices) FVector3f(Vertices[i]);
					}
				}
				else if( (PrevDist >= -Thresh) && (PrevDist < +Thresh) )
				{
					// Previous point lies on plane.
					if (Status == V_FRONT)
					{
						new(FrontPoly->Vertices) FVector3f(Vertices[j]);
						new(FrontPoly->Vertices) FVector3f(Vertices[i]);
					}
					else
					{
						new(BackPoly->Vertices) FVector3f(Vertices[j]);
						new(BackPoly->Vertices) FVector3f(Vertices[i]);
					}
				}
				else
				{
					// Intersection point is in between.
					Intersection = (FVector)FMath::LinePlaneIntersection(Vertices[j],Vertices[i],PlaneBase,PlaneNormal);

					if( PrevStatus == V_FRONT )
					{
						new(FrontPoly->Vertices) FVector3f(Intersection);
						new(BackPoly->Vertices) FVector3f(Intersection);
						new(BackPoly->Vertices) FVector3f(Vertices[i]);
					}
					else
					{
						new(BackPoly->Vertices) FVector3f(Intersection);
						new(FrontPoly->Vertices) FVector3f(Intersection);
						new(FrontPoly->Vertices) FVector3f(Vertices[i]);
					}
				}
			}
			else
			{
	    		if (Status==V_FRONT) new(FrontPoly->Vertices)FVector3f(Vertices[i]);
	    		else                 new(BackPoly->Vertices)FVector3f(Vertices[i]);
			}
			j          = i;
			PrevStatus = Status;
		}

		// Handle possibility of sliver polys due to precision errors.
		if( FrontPoly->Fix()<3 )
		{
// 			UE_LOGF(LogPolygon, Warning, "FPoly::SplitWithPlane: Ignored front sliver" );
			return SP_Back;
		}
		else if( BackPoly->Fix()<3 )
	    {
// 			UE_LOGF(LogPolygon, Warning, "FPoly::SplitWithPlane: Ignored back sliver" );
			return SP_Front;
		}
		else return SP_Split;
	}
}


int32 FPoly::SplitWithNode
(
	const UModel	*Model,
	int32				iNode,
	FPoly			*FrontPoly,
	FPoly			*BackPoly,
	int32				VeryPrecise
) const
{
	const FBspNode &Node = Model->Nodes[iNode       ];
	const FBspSurf &Surf = Model->Surfs[Node.iSurf  ];

	return SplitWithPlane
	(
		Model->Points [Model->Verts[Node.iVertPool].pVertex],
		Model->Vectors[Surf.vNormal],
		FrontPoly, 
		BackPoly, 
		VeryPrecise
	);
}


int32 FPoly::SplitWithPlaneFast
(
	const FPlane&	Plane,
	FPoly*			FrontPoly,
	FPoly*			BackPoly
) const
{
	FMemMark MemMark(FMemStack::Get());
	enum EPlaneClassification
	{
		V_FRONT=0,
		V_BACK=1
	};
	EPlaneClassification Status,PrevStatus;
	EPlaneClassification* VertStatus = new(FMemStack::Get()) EPlaneClassification[Vertices.Num()];
	int32 Front=0,Back=0;

	EPlaneClassification* StatusPtr = &VertStatus[0];
	for( int32 i=0; i<Vertices.Num(); i++ )
	{
		float Dist = Plane.PlaneDot((FVector)Vertices[i]);
		if( Dist >= 0.f )
		{
			*StatusPtr++ = V_FRONT;
			if( Dist > +UE_THRESH_SPLIT_POLY_WITH_PLANE )
				Front=1;
		}
		else
		{
			*StatusPtr++ = V_BACK;
			if( Dist < -UE_THRESH_SPLIT_POLY_WITH_PLANE )
				Back=1;
		}
	}
	ESplitType Result;
	if( !Front )
	{
		if( Back ) Result = SP_Back;
		else       Result = SP_Coplanar;
	}
	else if( !Back )
	{
		Result = SP_Front;
	}
	else
	{
		// Split.
		if( FrontPoly )
		{
			const FVector3f *V  = Vertices.GetData();
			const FVector3f *W  = V + Vertices.Num()-1;
			FVector3f *V1       = FrontPoly->Vertices.GetData();
			FVector3f *V2       = BackPoly ->Vertices.GetData();
			StatusPtr         = &VertStatus        [0];
			PrevStatus        = VertStatus         [Vertices.Num()-1];

			for( int32 i=0; i<Vertices.Num(); i++ )
			{
				Status = *StatusPtr++;
				if( Status != PrevStatus )
				{
					// Crossing.
					const FVector3f& Intersection = FMath::LinePlaneIntersection( *W, *V, (FPlane4f)Plane );
					new(FrontPoly->Vertices) FVector3f(Intersection);
					new(BackPoly->Vertices) FVector3f(Intersection);
					if( PrevStatus == V_FRONT )
					{
						new(BackPoly->Vertices) FVector3f(*V);
					}
					else
					{
						new(FrontPoly->Vertices) FVector3f(*V);
					}
				}
				else if( Status==V_FRONT )
				{
					new(FrontPoly->Vertices) FVector3f(*V);
				}
				else
				{
					new(BackPoly->Vertices) FVector3f(*V);
				}

				PrevStatus = Status;
				W          = V++;
			}
			FrontPoly->Base			= Base;
			FrontPoly->Normal		= Normal;
			FrontPoly->PolyFlags	= PolyFlags;

			BackPoly->Base			= Base;
			BackPoly->Normal		= Normal;
			BackPoly->PolyFlags		= PolyFlags;
		}
		Result = SP_Split;
	}

	MemMark.Pop();

	return Result;
}


int32 FPoly::CalcNormal( bool bSilent )
{
	Normal = FVector3f::ZeroVector;
	for( int32 i=2; i<Vertices.Num(); i++ )
		Normal += (Vertices[i-1] - Vertices[0]) ^ (Vertices[i] - Vertices[0]);

	if( Normal.SizeSquared() < (float)UE_THRESH_ZERO_NORM_SQUARED )
	{
// 		if( !bSilent )
// 			UE_LOGF(LogPolygon, Warning, "FPoly::CalcNormal: Zero-area polygon" );
		return 1;
	}
	Normal.Normalize();
	return 0;
}


void FPoly::Transform
(
	const FVector3f&		PostAdd
)
{
	int32 		i;

	Base += PostAdd;
	for( i=0; i<Vertices.Num(); i++ )
		Vertices[i] += PostAdd;
}


void FPoly::Rotate
(
	const FRotator3f&		Rotation
)
{
	const FRotationMatrix44f RotMatrix(Rotation);

	// Rotate the vertices.
	for (int32 Vertex = 0; Vertex < Vertices.Num(); Vertex++)
	{
		Vertices[Vertex] = RotMatrix.TransformVector(Vertices[Vertex]);
	}

	Base = RotMatrix.TransformVector(Base);

	// Rotate the texture vectors.
	TextureU = RotMatrix.TransformVector(TextureU);
	TextureV = RotMatrix.TransformVector(TextureV);

	// Rotate the normal.
	Normal = RotMatrix.TransformVector(Normal);
	Normal = Normal.GetSafeNormal();
}


void FPoly::Scale
(
	const FVector3f&		Scale
)
{
	if (Scale.X == 1.f &&
		Scale.Y == 1.f &&
		Scale.Z == 1.f)
	{
		return;
	}

	// Scale the vertices.
	for (int32 Vertex = 0; Vertex < Vertices.Num(); Vertex++)
	{
		Vertices[Vertex] *= Scale;
	}

	Base *= Scale;

	// Scale the texture vectors.
	TextureU /= Scale;
	TextureV /= Scale;

	// Recalculate the normal. Non-uniform scale or mirroring requires this.
	CalcNormal(true);
}


int32 FPoly::RemoveColinears()
{
	FMemStack& LocalMemStack = FMemStack::Get();
	FMemMark MemMark(LocalMemStack);
	FVector3f* SidePlaneNormal = new(LocalMemStack) FVector3f[Vertices.Num()];
	FVector3f  Side;
	int32      i,j;
	bool Result = true;

	for( i=0; i<Vertices.Num(); i++ )
	{
		j=(i+Vertices.Num()-1)%Vertices.Num();

		// Create cutting plane perpendicular to both this side and the polygon's normal.
		Side = Vertices[i] - Vertices[j];
		SidePlaneNormal[i] = Side ^ Normal;

		if( !SidePlaneNormal[i].Normalize() )
		{
			// Eliminate these nearly identical points.
			Vertices.RemoveAt(i,1);
			if(Vertices.Num() < 3)
			{
				// Collapsed.
				Vertices.Empty();
				Result = false;
				break;
			}
			i--;
		}
	}
	if(Result)
	{
		for( i=0; i<Vertices.Num(); i++ )
		{
			j=(i+1)%Vertices.Num();

			if( FVector3f::PointsAreNear(SidePlaneNormal[i],SidePlaneNormal[j], UE_FLOAT_NORMAL_THRESH) )
			{
				// Eliminate colinear points.
				FMemory::Memcpy (&SidePlaneNormal[i],&SidePlaneNormal[i+1],(Vertices.Num()-(i+1)) * sizeof (FVector3f));
				Vertices.RemoveAt(i,1);
				if(Vertices.Num() < 3)
				{
					// Collapsed.
					Vertices.Empty();
					Result = false;
					break;
				}
				i--;
			}
			else
			{
				switch( SplitWithPlane (Vertices[i],SidePlaneNormal[i],NULL,NULL,0) )
				{
					case SP_Front:
						Result = false;
						break;
					case SP_Split:
						Result = false;
						break;
					// SP_BACK: Means it's convex
					// SP_COPLANAR: Means it's probably convex (numerical precision)
				}
				if(!Result)
				{
					break;
				}
			}
		}
	}

	MemMark.Pop();

	return Result; // Ok.
}


bool FPoly::DoesLineIntersect( FVector Start, FVector End, FVector* Intersect )
{
	// If the ray doesn't cross the plane, don't bother going any further.
	const float DistStart = FVector::PointPlaneDist( Start, (FVector)Vertices[0], (FVector)Normal );
	const float DistEnd = FVector::PointPlaneDist( End, (FVector)Vertices[0], (FVector)Normal );

	if( (DistStart < 0 && DistEnd < 0) || (DistStart > 0 && DistEnd > 0 ) )
	{
		return 0;
	}

	// Get the intersection of the line and the plane.
	FVector Intersection = FMath::LinePlaneIntersection(Start,End,(FVector)Vertices[0],(FVector)Normal);
	if( Intersect )	*Intersect = Intersection;
	if( Intersection == Start || Intersection == End )
	{
		return 0;
	}

	// Check if the intersection point is actually on the poly.
	return OnPoly( Intersection );
}


bool FPoly::OnPoly( FVector InVtx )
{
	FVector  SidePlaneNormal;
	FVector3f  Side;

	for( int32 x = 0 ; x < Vertices.Num() ; x++ )
	{
		// Create plane perpendicular to both this side and the polygon's normal.
		Side = Vertices[x] - Vertices[(x-1 < 0 ) ? Vertices.Num()-1 : x-1 ];
		SidePlaneNormal = FVector(Side ^ Normal);
		SidePlaneNormal.Normalize();

		// If point is not behind all the planes created by this polys edges, it's outside the poly.
		if( FVector::PointPlaneDist( InVtx, (FVector)Vertices[x], SidePlaneNormal ) > UE_THRESH_POINT_ON_PLANE )
		{
			return 0;
		}
	}

	return 1;
}


void FPoly::InsertVertex( int32 InPos, FVector InVtx )
{
	check( InPos <= Vertices.Num() );

	Vertices.Insert((FVector3f)InVtx,InPos);
}


void FPoly::RemoveVertex( FVector InVtx )
{
	Vertices.Remove((FVector3f)InVtx);
}



bool FPoly::IsCoplanar()
{
	// 3 or fewer vertices is automatically coplanar

	if( Vertices.Num() <= 3 )
	{
		return 1;
	}

	CalcNormal(1);

	// Note that we do not need to test the first vertex since the OnPlane() check uses it as the "reference" vertex, and always return true for it. 
	for (int32 x = 1; x < Vertices.Num(); ++x)
	{
		if( !OnPlane( (FVector)Vertices[x] ) )
		{
			return 0;
		}
	}

	// If we got this far, the poly has to be coplanar.

	return 1;
}



bool FPoly::IsConvex()
{
	const int32 VerticesNum = Vertices.Num();
	if (VerticesNum > 2)
	{
		FVector3f Vertex1 = Vertices[VerticesNum - 2];
		FVector3f Vertex2 = Vertices[VerticesNum - 1];
		FVector3f EdgeVertex1Vertex2 = (Vertex2 - Vertex1).GetSafeNormal();

		for (int32 Index3 = 0; Index3 < VerticesNum; ++Index3)
		{
			const FVector3f EdgeVertex1Vertex2Normal = FVector3f::CrossProduct(EdgeVertex1Vertex2, Normal);
			const FVector3f Vertex3 = Vertices[Index3];
			const FVector3f EdgeVertex2Vertex3 = (Vertex3 - Vertex2).GetSafeNormal();

			if (UE_KINDA_SMALL_NUMBER < FVector3f::DotProduct(EdgeVertex1Vertex2Normal, EdgeVertex2Vertex3))
			{
				return false;
			}

			Vertex1 = Vertex2;
			Vertex2 = Vertex3;
			EdgeVertex1Vertex2 = EdgeVertex2Vertex3;
		}
	}

	return true;
}



int32 FPoly::Triangulate( ABrush* InOwnerBrush, TArray<FPoly>& OutTriangles )
{
#if WITH_EDITOR

	if( Vertices.Num() < 3 )
	{
		return 0;
	}

	FClipSMPolygon Polygon(0);

	for( int32 v = 0 ; v < Vertices.Num() ; ++v )
	{
		FClipSMVertex vtx;
		vtx.Pos = Vertices[v];

		// Init other data so that VertsAreEqual won't compare garbage
		vtx.TangentX = FVector3f::ZeroVector;
		vtx.TangentY = FVector3f::ZeroVector;
		vtx.TangentZ = FVector3f::ZeroVector;
		vtx.Color = FColor(0, 0, 0);
		for( int32 uvIndex=0; uvIndex<UE_ARRAY_COUNT(vtx.UVs); ++uvIndex )
		{
			vtx.UVs[uvIndex] = FVector2D(0.f, 0.f);
		}


		Polygon.Vertices.Add( vtx );
	}

	Polygon.FaceNormal = (FVector)Normal;

	// Attempt to triangulate this polygon
	TArray<FClipSMTriangle> Triangles;
	if( FGeomTools::TriangulatePoly( Triangles, Polygon ) )
	{
		// Create a set of FPolys from the triangles

		OutTriangles.Empty();
		FPoly TrianglePoly;

		for( int32 p = 0 ; p < Triangles.Num() ; ++p )
		{
			FClipSMTriangle* tri = &(Triangles[p]);

			TrianglePoly.Init();
			TrianglePoly.Base = tri->Vertices[0].Pos;

			TrianglePoly.Vertices.Add( tri->Vertices[0].Pos );
			TrianglePoly.Vertices.Add( tri->Vertices[1].Pos );
			TrianglePoly.Vertices.Add( tri->Vertices[2].Pos );

			if( TrianglePoly.Finalize( InOwnerBrush, 0 ) == 0 )
			{
				OutTriangles.Add( TrianglePoly );
			}
		}
	}
#endif

	return OutTriangles.Num();
}


int32 FPoly::GetVertexIndex( FVector3f& InVtx )
{
	int32 idx = INDEX_NONE;

	for( int32 v = 0 ; v < Vertices.Num() ; ++v )
	{
		if( Vertices[v] == InVtx )
		{
			idx = v;
			break;
		}
	}

	return idx;
}



FVector FPoly::GetMidPoint()
{
	FVector mid(0,0,0);

	for( int32 v = 0 ; v < Vertices.Num() ; ++v )
	{
		mid += (FVector)Vertices[v];
	}

	return mid / Vertices.Num();
}


FPoly FPoly::BuildInfiniteFPoly(const FPlane& InPlane)
{
	FVector Axis1, Axis2;

	// Find two non-problematic axis vectors.
	InPlane.FindBestAxisVectors( Axis1, Axis2 );

	// Set up the FPoly.
	FPoly EdPoly;
	EdPoly.Init();
	EdPoly.Normal.X    = InPlane.X;
	EdPoly.Normal.Y    = InPlane.Y;
	EdPoly.Normal.Z    = InPlane.Z;
	EdPoly.Base        = EdPoly.Normal * InPlane.W;
	EdPoly.Vertices.Add( EdPoly.Base + (FVector3f)Axis1*UE_OLD_HALF_WORLD_MAX + (FVector3f)Axis2*UE_OLD_HALF_WORLD_MAX );
	EdPoly.Vertices.Add( EdPoly.Base - (FVector3f)Axis1*UE_OLD_HALF_WORLD_MAX + (FVector3f)Axis2*UE_OLD_HALF_WORLD_MAX );
	EdPoly.Vertices.Add( EdPoly.Base - (FVector3f)Axis1*UE_OLD_HALF_WORLD_MAX - (FVector3f)Axis2*UE_OLD_HALF_WORLD_MAX );
	EdPoly.Vertices.Add( EdPoly.Base + (FVector3f)Axis1*UE_OLD_HALF_WORLD_MAX - (FVector3f)Axis2*UE_OLD_HALF_WORLD_MAX );

	return EdPoly;
}

#if WITH_EDITOR
FPoly FPoly::BuildAndCutInfiniteFPoly(const FPlane& InPlane, const TArray<FPlane>& InCutPlanes, ABrush* InOwnerBrush)
{
	FPoly PolyMerged = BuildInfiniteFPoly( InPlane );
	PolyMerged.Finalize( InOwnerBrush, 1 );

	FPoly Front, Back;
	int32 result;

	for( int32 p = 0 ; p < InCutPlanes.Num() ; ++p )
	{
		const FPlane* Plane = &InCutPlanes[p];

		result = PolyMerged.SplitWithPlane(FVector3f(Plane->GetSafeNormal() * Plane->W), (FVector3f)Plane->GetSafeNormal(), &Front, &Back, 1 );

		if( result == SP_Split )
		{
			PolyMerged = Back;
		}
	}

	PolyMerged.Reverse();

	return PolyMerged;
}
#endif // WITH_EDITOR

bool FPoly::OnPlane( FVector InVtx )
{
	return ( FMath::Abs( FVector::PointPlaneDist( InVtx, (FVector)Vertices[0], (FVector)Normal ) ) < UE_THRESH_POINT_ON_PLANE );
}


int32 FPoly::Split( const FVector3f &InNormal, const FVector3f &InBase )
{
	// Split it.
	FPoly Front, Back;
	Front.Init();
	Back.Init();
	switch( SplitWithPlaneFast( FPlane((FVector)InBase, (FVector)InNormal), &Front, &Back ))
	{
		case SP_Back:
			return 0;
		case SP_Split:
			*this = Front;
			return Vertices.Num();
		default:
			return Vertices.Num();
	}
}

#if WITH_EDITOR

int32 FPoly::Finalize( ABrush* InOwner, int32 NoError )
{
	// Check for problems.
	Fix();
	if( Vertices.Num()<3 )
	{
		// Since we don't have enough vertices, remove this polygon from the brush
		check( InOwner );
		for( int32 p = 0 ; p < InOwner->Brush->Polys->Element.Num() ; ++p )
		{
			if( InOwner->Brush->Polys->Element[p] == *this )
			{
				InOwner->Brush->Polys->Element.RemoveAt(p);
				break;
			}
		}
	
// 		UE_LOGF(LogPolygon, Warning, "FPoly::Finalize: Not enough vertices (%i)", Vertices.Num() );
		if( NoError )
			return -1;
		else
		{
// 			UE_LOGF(LogPolygon, Log,  "FPoly::Finalize: Not enough vertices (%i) : polygon removed from brush", Vertices.Num() );
			return -2;
		}
	}

	// Compute normal from cross-product and normalize it.
	if( CalcNormal() )
	{
// 		UE_LOGF(LogPolygon, Warning, "FPoly::Finalize: Normalization failed, verts=%i, size=%f", Vertices.Num(), Normal.Size() );
		if( NoError )
		{
			return -1;
		}
		else
		{
			UE_LOGF(LogPolygon, Error, "FPoly::Finalize: Normalization failed, verts=%d, size=%f", Vertices.Num(), Normal.Size());
			for (int32 VertIdx = 0; VertIdx < Vertices.Num(); ++VertIdx)
			{
				UE_LOGF(LogPolygon, Error, "Vertex %d: <%.6f, %.6f, %.6f>", VertIdx, Vertices[VertIdx].X, Vertices[VertIdx].Y, Vertices[VertIdx].Z);
			}
			ensure(false);
		}
	}

	// If texture U and V coordinates weren't specified, generate them.
	if( TextureU.IsZero() && TextureV.IsZero() )
	{
		for( int32 i=1; i<Vertices.Num(); i++ )
		{
			TextureU = ((Vertices[0] - Vertices[i]) ^ Normal).GetSafeNormal();
			TextureV = (Normal ^ TextureU).GetSafeNormal();
			if( TextureU.SizeSquared()!=0 && TextureV.SizeSquared()!=0 )
				break;
		}
	}
	return 0;
}

template<typename ArrayType>
void FPoly::OptimizeIntoConvexPolys(ABrush* InOwnerBrush, ArrayType& InPolygons)
{
	// In this implementation, we find all coplanar polygons that share an edge and merge them together provided they form a convex polygon.

	// Flag that indicates that we might have missed some potential neighbors after we jumped over one but then found another one.
	// If this is the case, we need to check everything again.
	bool bMightHaveMissedSomeNeighbors;

	do
	{
		bMightHaveMissedSomeNeighbors = false;

		// We iterate over the polygons in reverse order. This is effectively the inverse of how polygons get split up during triangulation.
		// Starting at the end, the current polygon is considered the "main" polygon, and we try to grow it as much as possible using polygons earlier in
		// the list called "neighbor" polygons. Whenever we jump over a potential neighbor polygon, and then find a polygon that is an actual neighbor, we
		// effectively need to check again since the one we jumped over earlier might now be connectable.

		for (int32 PolyMainIndex = InPolygons.Num() - 1; PolyMainIndex > 0; --PolyMainIndex)
		{
			FPoly* PolyMain = &InPolygons[PolyMainIndex];

			// Iterate over all potential neighbor polygons, again in reverse order. 
			for (int32 PolyNeighborIndex = PolyMainIndex - 1; PolyNeighborIndex >= 0; --PolyNeighborIndex)
			{
				const FPoly* PolyNeighbor = &InPolygons[PolyNeighborIndex];

				if (PolyMain->Normal.Equals(PolyNeighbor->Normal))
				{
					// See if PolyNeighbor is sharing an edge with PolyMain.

					const int32 Poly1Num = PolyMain->Vertices.Num();
					const int32 Poly2Num = PolyNeighbor->Vertices.Num();

					// Shared edge information for both polygons; this will be uninitialized if there is no shared edge.
					FVector3f EdgeVtxA, EdgeVtxB; // Position for edge vertices A and B
					int32 Poly1IndexA = INDEX_NONE, Poly1IndexB = INDEX_NONE; // Indices for edge vertices A and B in first polygon, PolyMain.
					int32 Poly2IndexA = INDEX_NONE, Poly2IndexB = INDEX_NONE; // Indices for edge vertices A and B in second polygon, PolyNeighbor.

					// Determine if there is a shared edge, and if yes, fills in the shared edge information for both polygons.
					// The edge vertices, EdgeVtxA and EdgeVtxB, are set to the average between the corresponding vertices in both polygons.
					const bool bSharedEdge = [PolyMain, PolyNeighbor, Poly1Num, Poly2Num, &EdgeVtxA, &EdgeVtxB,
						&Poly1IndexA, &Poly1IndexB, &Poly2IndexA, &Poly2IndexB]
					{
						if (Poly1Num > 1 && Poly2Num > 1)
						{
							for (Poly1IndexA = Poly1Num - 1, Poly1IndexB = 0; Poly1IndexB < Poly1Num; Poly1IndexA = Poly1IndexB++)
							{
								// For each edge in the first polygon ...
								const FVector3f& Poly1Vtx1 = PolyMain->Vertices[Poly1IndexA];
								const FVector3f& Poly1Vtx2 = PolyMain->Vertices[Poly1IndexB];

								for (Poly2IndexB = Poly2Num - 1, Poly2IndexA = 0; Poly2IndexA < Poly2Num; Poly2IndexB = Poly2IndexA++)
								{
									// ... and for each edge in the second polygon ...
									const FVector3f& Poly2Vtx1 = PolyNeighbor->Vertices[Poly2IndexB];
									const FVector3f& Poly2Vtx2 = PolyNeighbor->Vertices[Poly2IndexA];

									// ... check if they have the same vertices for opposite orientation, i.e. a shared edge.
									if (Poly1Vtx1.Equals(Poly2Vtx2) && Poly1Vtx2.Equals(Poly2Vtx1))
									{
										EdgeVtxA = (Poly1Vtx1 + Poly2Vtx2) * 0.5f;
										EdgeVtxB = (Poly1Vtx2 + Poly2Vtx1) * 0.5f;
										return true;
									}
								}
							}
						}
						return false;
					}();

					if (bSharedEdge)
					{
						// Found a shared edge. Merge both polygons around the shared edge and check if the result is coplanar and convex.

						// Note that adding the vertices and polygons in this very order keeps the repeated execution of "Triangulate" and "Optimize"
						// more stable, i.e. when you have a polygon and click "Triangulate" followed by "Optimize" it will more likely give you the
						// same polygon with the same vertex order and the same UVs.

						FPoly PolyMerged;
						int32 PolyMergedNum = Poly1Num + Poly2Num - 2;
						PolyMerged.Vertices.SetNumUninitialized(PolyMergedNum);

						int32 PolyMergedIndex = 0;
						for (int32 V = (Poly2IndexA + 1) % Poly2Num; V != Poly2IndexB; V = (V + 1) % Poly2Num)
						{
							PolyMerged.Vertices[PolyMergedIndex++] = PolyNeighbor->Vertices[V];
						}
						PolyMerged.Vertices[PolyMergedIndex++] = EdgeVtxB;
						for (int32 V = (Poly1IndexB + 1) % Poly1Num; V != Poly1IndexA; V = (V + 1) % Poly1Num)
						{
							PolyMerged.Vertices[PolyMergedIndex++] = PolyMain->Vertices[V];
						}
						PolyMerged.Vertices[PolyMergedIndex] = EdgeVtxA;
						checkSlow(++PolyMergedIndex == PolyMerged.Vertices.Num());

						// Computing the normal is not cheap, but we need it for the convexity test. Thus, we just use the normals we already have since they
						// are close enough anyway. The proper normal will be computed during the later call to `FPoly::Finalize()` anyway.
						PolyMerged.Normal = (PolyMain->Normal + PolyNeighbor->Normal) * 0.5f;

						// Check convexity equivalent to FPoly::IsConvex(), but also remove collinear edges in the process.
						const bool bIsConvex = [&PolyMerged, &PolyMergedNum]
						{
							if (PolyMergedNum > 2)
							{
								FVector3f Vertex1 = PolyMerged.Vertices[PolyMergedNum - 2];
								int32 Index2 = PolyMergedNum - 1;
								FVector3f Vertex2 = PolyMerged.Vertices[Index2];
								FVector3f EdgeVertex1Vertex2 = (Vertex2 - Vertex1).GetSafeNormal();

								for (int32 Index3 = 0; Index3 < PolyMergedNum; ++Index3)
								{
									const FVector3f Vertex3 = PolyMerged.Vertices[Index3];
									const FVector3f EdgeVertex2Vertex3 = (Vertex3 - Vertex2).GetSafeNormal();

									// Neighboring edges are collinear if they point into the same direction.
									const bool bIsCollinear = EdgeVertex1Vertex2.Equals(EdgeVertex2Vertex3, UE_KINDA_SMALL_NUMBER);

									if (bIsCollinear)
									{
										// Collapse collinear edges by removing vertex 2.
										PolyMerged.Vertices.RemoveAt(Index2);
										--PolyMergedNum;

										// Vertex 1 stays the same, but recompute the edge vector.
										EdgeVertex1Vertex2 = (Vertex3 - Vertex1).GetSafeNormal();
									}
									else
									{
										// Check convexity for neighboring non-collinear edges.
										const FVector3f EdgeVertex1Vertex2Normal = FVector3f::CrossProduct(EdgeVertex1Vertex2, PolyMerged.Normal);
										if (UE_KINDA_SMALL_NUMBER < FVector3f::DotProduct(EdgeVertex1Vertex2Normal, EdgeVertex2Vertex3))
										{
											return false;
										}

										// Vertex 1 moves to vertex 2, and the edge vector gets updated accordingly.
										Vertex1 = Vertex2;
										EdgeVertex1Vertex2 = EdgeVertex2Vertex3;
									}

									// Vertex 2 moves to vertex 3.
									Index2 = Index3;
									Vertex2 = Vertex3;
								}
							}

							return true;
						}();

						// If the merged polygon is convex and coplanar, finalize it, and enter it into the list.
						// We explicitly call FPoly::IsCoplanar() here even though our own test already established that to make sure that we don't end up
						// in infinite loops of failing coplanar check, triangulation, and optimization due to precision issues in the coplanar check.
						if (bIsConvex && PolyMerged.IsCoplanar() && PolyMerged.Finalize(InOwnerBrush, 1) == 0)
						{
							// If the main polygon is not last one or the neighbor polygon is not the second to last one, we might have missed neighbors.
							bMightHaveMissedSomeNeighbors |= PolyMainIndex != InPolygons.Num() - 1 || PolyNeighborIndex != PolyMainIndex - 1;

							// Remove the neighbor polygon, fix up the main polygon index, and make the merged polygon the new main polygon.
							InPolygons.RemoveAt(PolyNeighborIndex);
							InPolygons[--PolyMainIndex] = PolyMerged;
							PolyMain = &InPolygons[PolyMainIndex];
						}
					}
				}
			}
		}
	}
	while (bMightHaveMissedSomeNeighbors);
}

template ENGINE_API void FPoly::OptimizeIntoConvexPolys<TArray<FPoly>>(ABrush*, TArray<FPoly>&);

#endif // WITH_EDITOR


int32 FPoly::Faces( const FPoly &Test ) const
{
	// Coplanar implies not facing.
	if( IsCoplanar( Test ) )
		return 0;

	// If this poly is frontfaced relative to all of Test's points, they're not facing.
	for( int32 i=0; i<Test.Vertices.Num(); i++ )
	{
		if( !IsBackfaced( Test.Vertices[i] ) )
		{
			// If Test is frontfaced relative to on or more of this poly's points, they're facing.
			for( i=0; i<Vertices.Num(); i++ )
				if( Test.IsBackfaced( Vertices[i] ) )
					return 1;
			return 0;
		}
	}
	return 0;
}

#if WITH_EDITOR
bool UPolys::Modify(bool bAlwaysMarkDirty)
{
	Super::Modify(bAlwaysMarkDirty);

	return !!GUndo; // we will make a broad assumption that if we have an undo buffer, we were saved in it
}
#endif

void UPolys::AddReferencedObjects(UObject* InThis, FReferenceCollector& Collector)
{
	UPolys* This = CastChecked<UPolys>(InThis);
	// Let GC know that we're referencing some Actor and Property objects
	for( TArray<FPoly>::TConstIterator ElementIt(This->Element); ElementIt; ++ElementIt )
	{
		FPoly PolyElement = *ElementIt;
		Collector.AddReferencedObject( PolyElement.Actor, This );
		Collector.AddReferencedObject( PolyElement.Material, This );
	}
	Super::AddReferencedObjects(This, Collector);
}

void UPolys::Serialize( FArchive& Ar )
{
	Super::Serialize( Ar );
	if( Ar.IsTransacting() )
	{
		Ar << Element;
	}
	else
	{
		if( Ar.IsLoading() && Ar.UEVer() < VER_UE4_BSP_UNDO_FIX )
		{
			Element.CountBytes(Ar);
			int32 DbNum = Element.Num(), DbMax = DbNum;
			Ar << DbNum << DbMax;

		
			UObject* ElementOwner = NULL;
			Ar << ElementOwner;

			if (Ar.IsLoading())
			{
				Element.Empty(DbNum);
				Element.AddZeroed(DbNum);
			}
			for (int32 i = 0; i < Element.Num(); i++)
			{
				Ar << Element[i];
			}
		}
		else
		{
			Ar << Element;
		}
	}
}
