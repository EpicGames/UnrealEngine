// Copyright Epic Games, Inc. All Rights Reserved.

#include "Rendering/PositionVertexBuffer.h"

#include "Components.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "RenderingThread.h"
#include "RenderUtils.h"
#include "RHIResourceReplace.h"
#include "StaticMeshVertexData.h"

/*-----------------------------------------------------------------------------
FPositionVertexBuffer
-----------------------------------------------------------------------------*/

/** The implementation of the static mesh position-only vertex data storage type. */
class FPositionVertexData :
	public TStaticMeshVertexData<FPositionVertex>
{
public:
	FPositionVertexData(bool InNeedsCPUAccess = false)
		: TStaticMeshVertexData<FPositionVertex>(InNeedsCPUAccess)
	{
	}
};


FPositionVertexBuffer::FPositionVertexBuffer():
	VertexData(NULL),
	Data(NULL),
	Stride(0),
	NumVertices(0)
{}

FPositionVertexBuffer::~FPositionVertexBuffer()
{
	CleanUp();
}

/** Delete existing resources */
void FPositionVertexBuffer::CleanUp()
{
	if (VertexData)
	{
		delete VertexData;
		VertexData = nullptr;
	}
}

void FPositionVertexBuffer::Init(uint32 InNumVertices, bool bInNeedsCPUAccess)
{
	NumVertices = InNumVertices;
	bNeedsCPUAccess = bInNeedsCPUAccess;

	// Allocate the vertex data storage type.
	AllocateData(bInNeedsCPUAccess);

	// Allocate the vertex data buffer.
	VertexData->ResizeBuffer(NumVertices);
	Data = NumVertices ? VertexData->GetDataPointer() : nullptr;
}

/**
* Initializes the buffer with the given vertices, used to convert legacy layouts.
* @param InVertices - The vertices to initialize the buffer with.
*/
void FPositionVertexBuffer::Init(const TArray<FStaticMeshBuildVertex>& InVertices, bool bInNeedsCPUAccess)
{
	const FConstMeshBuildVertexView VertexView = MakeConstMeshBuildVertexView(InVertices);
	Init(VertexView, bInNeedsCPUAccess);
}

void FPositionVertexBuffer::Init(const FConstMeshBuildVertexView& InVertices, bool bInNeedsCPUAccess)
{
	Init(InVertices.Position.Num(), bInNeedsCPUAccess);

	// Copy the vertex positions into the buffer.
	for (int32 VertexIndex = 0; VertexIndex < InVertices.Position.Num(); VertexIndex++)
	{
		VertexPosition(VertexIndex) = InVertices.Position[VertexIndex];
	}
}

/**
* Initializes this vertex buffer with the contents of the given vertex buffer.
* @param InVertexBuffer - The vertex buffer to initialize from.
*/
void FPositionVertexBuffer::Init(const FPositionVertexBuffer& InVertexBuffer, bool bInNeedsCPUAccess)
{
	bNeedsCPUAccess = bInNeedsCPUAccess;
	if ( InVertexBuffer.GetNumVertices() )
	{
		Init(InVertexBuffer.GetNumVertices(), bInNeedsCPUAccess);

		check( Stride == InVertexBuffer.GetStride() );

		const uint8* InData = InVertexBuffer.Data;
		FMemory::Memcpy( Data, InData, Stride * NumVertices );
	}
}

void FPositionVertexBuffer::Init(const TArray<FVector3f>& InPositions, bool bInNeedsCPUAccess)
{
	NumVertices = InPositions.Num();
	bNeedsCPUAccess = bInNeedsCPUAccess;
	if ( NumVertices )
	{
		AllocateData(bInNeedsCPUAccess);
		check( Stride == InPositions.GetTypeSize() );
		VertexData->ResizeBuffer(NumVertices);
		Data = VertexData->GetDataPointer();
		FMemory::Memcpy( Data, InPositions.GetData(), Stride * NumVertices );
	}
}

bool FPositionVertexBuffer::AppendVertices( const FStaticMeshBuildVertex* Vertices, const uint32 NumVerticesToAppend )
{
	const uint64 TotalNumVertices = (uint64)NumVertices + (uint64)NumVerticesToAppend;
	if (!ensureMsgf(TotalNumVertices < INT32_MAX, TEXT("FPositionVertexBuffer::AppendVertices adding %u to %u vertices exceeds INT32_MAX limit"), NumVerticesToAppend, NumVertices))
	{
		return false;
	}

	if (VertexData == nullptr && NumVerticesToAppend > 0)
	{
		// Allocate the vertex data storage type if the buffer was never allocated before
		AllocateData(bNeedsCPUAccess);
	}

	if( NumVerticesToAppend > 0 )
	{
		check( VertexData != nullptr );
		check( Vertices != nullptr );

		const uint32 FirstDestVertexIndex = NumVertices;
		NumVertices += NumVerticesToAppend;
		VertexData->ResizeBuffer( NumVertices );
		if( NumVertices > 0 )
		{
			Data = VertexData->GetDataPointer();

			// Copy the vertices into the buffer.
			for( uint32 VertexIter = 0; VertexIter < NumVerticesToAppend; ++VertexIter )
			{
				const FStaticMeshBuildVertex& SourceVertex = Vertices[ VertexIter ];

				const uint32 DestVertexIndex = FirstDestVertexIndex + VertexIter;
				VertexPosition( DestVertexIndex ) = SourceVertex.Position;
			}
		}
	}

	return true;
}

/**
 * Serializer
 *
 * @param	Ar					Archive to serialize with
 * @param	bInNeedsCPUAccess	Whether the elements need to be accessed by the CPU
 */
void FPositionVertexBuffer::Serialize( FArchive& Ar, bool bInNeedsCPUAccess )
{
	bNeedsCPUAccess = bInNeedsCPUAccess;

	SerializeMetaData(Ar);

	if(Ar.IsLoading())
	{
		// Allocate the vertex data storage type.
		AllocateData( bInNeedsCPUAccess );
	}

	if(VertexData != NULL)
	{
		// Serialize the vertex data.
		VertexData->Serialize(Ar);

		// Make a copy of the vertex data pointer.
		Data = NumVertices ? VertexData->GetDataPointer() : nullptr;
	}
}

void FPositionVertexBuffer::SerializeMetaData(FArchive& Ar)
{
	Ar << Stride << NumVertices;
}

void FPositionVertexBuffer::SetMetaData(uint32 InStride, uint32 InNumVertices)
{
	Stride = InStride;
	NumVertices = InNumVertices;
}

void FPositionVertexBuffer::ClearMetaData()
{
	Stride = NumVertices = 0;
}

bool FPositionVertexBuffer::GetAllowCPUAccess() const
{
	return VertexData ? VertexData->GetAllowCPUAccess() : false;
}

/**
* Specialized assignment operator, only used when importing LOD's.  
*/
void FPositionVertexBuffer::operator=(const FPositionVertexBuffer &Other)
{
	//VertexData doesn't need to be allocated here because Build will be called next,
	VertexData = NULL;
}

FBufferRHIRef FPositionVertexBuffer::CreateRHIBuffer(FRHICommandListBase& RHICmdList)
{
	return FRenderResource::CreateRHIBuffer(RHICmdList, VertexData, NumVertices, BUF_Static | BUF_ShaderResource, TEXT("FPositionVertexBuffer"));
}

void FPositionVertexBuffer::InitRHIForStreaming(FRHIBuffer* IntermediateBuffer, FRHIResourceReplaceBatcher& Batcher)
{
	check(VertexBufferRHI);
	if (IntermediateBuffer)
	{
		Batcher.EnqueueReplace(VertexBufferRHI, IntermediateBuffer);
	}
}

void FPositionVertexBuffer::ReleaseRHIForStreaming(FRHIResourceReplaceBatcher& Batcher)
{
	check(VertexBufferRHI);
	Batcher.EnqueueReplace(VertexBufferRHI, nullptr);
}

void FPositionVertexBuffer::InitRHI(FRHICommandListBase& RHICmdList)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPositionVertexBuffer::InitRHI);

	VertexBufferRHI = CreateRHIBuffer(RHICmdList);

	// Always create SRV if VertexBufferRHI is valid, as several systems rely on having position SRV. Memory impact has been measured to be minimal.
	if (VertexBufferRHI)
	{
		PositionComponentSRV = RHICmdList.CreateShaderResourceView(
			VertexBufferRHI, 
			FRHIViewDesc::CreateBufferSRV()
				.SetType(FRHIViewDesc::EBufferType::Typed)
				.SetFormat(PF_R32_FLOAT));
	}
}

void FPositionVertexBuffer::ReleaseRHI()
{
	PositionComponentSRV.SafeRelease();
	FVertexBuffer::ReleaseRHI();
}

void FPositionVertexBuffer::AllocateData( bool bInNeedsCPUAccess /*= true*/ )
{
	// Clear any old VertexData before allocating.
	CleanUp();

	VertexData = new FPositionVertexData(bInNeedsCPUAccess);
	// Calculate the vertex stride.
	Stride = VertexData->GetStride();
}

void FPositionVertexBuffer::BindPositionVertexBuffer(const FVertexFactory* VertexFactory, FStaticMeshDataType& StaticMeshData) const
{
	StaticMeshData.PositionComponent = FVertexStreamComponent(
		this,
		STRUCT_OFFSET(FPositionVertex, Position),
		GetStride(),
		VET_Float3
	);
	StaticMeshData.PositionComponentSRV = PositionComponentSRV;
}
