// Copyright Epic Games, Inc. All Rights Reserved.

#include "Engine/PostProcessVolume.h"
#include "Engine/BlendableInterface.h"
#include "Engine/CollisionProfile.h"
#include "Engine/Engine.h"
#include "Components/BrushComponent.h"
#include "EngineUtils.h"
#include "SceneInterface.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PostProcessVolume)

APostProcessVolume::APostProcessVolume(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	GetBrushComponent()->SetCollisionProfileName(UCollisionProfile::NoCollision_ProfileName);
	// post process volume needs physics data for trace
	GetBrushComponent()->bAlwaysCreatePhysicsState = true;
	GetBrushComponent()->Mobility = EComponentMobility::Movable;
	
	bEnabled = true;
	BlendRadius = 100.0f;
	BlendWeight = 1.0f;
}

#if WITH_EDITOR

// This delegate is needed to handle update of sort order when a post process volume's dimensions are changed in the editor.
static FDelegateHandle GPostProcessVolumeActorMovedDelegateHandle;

// Count of post process volume actors using the delegate above.  When this goes to zero, we can free the delegate.
static int32 GPostProcessVolumeDelegateCount = 0;

// Post process volumes are re-inserted to update their sort order following an editor move operation, as changes to the transform
// can affect the bounds used as part of the sort key.  By design, outside the editor, sorting is NOT updated after insertion, to
// avoid unexpected changes in order.  This would only matter if the post process volume is dynamically moved, which is likely
// rare to begin with, but we'd rather such movement not later affect sorting.
static void PostProcessVolumeOnEditorActorMoved(AActor* InActor)
{
	APostProcessVolume* VolumeActor = Cast<APostProcessVolume>(InActor);
	if (VolumeActor)
	{
		UWorld* World = VolumeActor->GetWorld();
		if (World && World->WorldType == EWorldType::Editor)
		{
			// Don't re-insert it if we didn't actually remove it
			if (World->RemovePostProcessVolume(VolumeActor))
			{
				World->AddPostProcessVolume(VolumeActor);
			}
		}
	}
}
#endif  // WITH_EDITOR

void APostProcessVolume::PostUnregisterAllComponents()
{
#if WITH_EDITOR
	if (GEngine)
	{
		GPostProcessVolumeDelegateCount--;
		check(GPostProcessVolumeDelegateCount >= 0);

		if (GPostProcessVolumeDelegateCount == 0)
		{
			GEngine->OnActorMoved().Remove(GPostProcessVolumeActorMovedDelegateHandle);

			GPostProcessVolumeActorMovedDelegateHandle.Reset();
		}
	}
#endif

	// Route clear to super first.
	Super::PostUnregisterAllComponents();
	// World will be NULL during exit purge.
	if (GetWorld())
	{
		GetWorld()->RemovePostProcessVolume(this);
	}
}

void APostProcessVolume::PostRegisterAllComponents()
{
	// Route update to super first.
	Super::PostRegisterAllComponents();
	GetWorld()->AddPostProcessVolume(this);

#if WITH_EDITOR
	if (GEngine)
	{
		// Add a delegate so moved post process volumes can update their sort order, as the Volume size is affected by the transform.
		if (GPostProcessVolumeDelegateCount == 0)
		{
			GPostProcessVolumeActorMovedDelegateHandle = GEngine->OnActorMoved().AddStatic(&PostProcessVolumeOnEditorActorMoved);
		}
		GPostProcessVolumeDelegateCount++;
	}
#endif
}

bool APostProcessVolume::IsPPVEnabled() const
{
#if WITH_EDITOR
	const bool bShowInEditor = GIsEditor ? !IsHiddenEd() : false;
	const bool bInGameWorld = GetWorld() && GetWorld()->UsesGameHiddenFlags();

	// bEnabled is the only thing we check in a game world. In the editor we also check the editor hidden flags
	// In a GameWorld we can't use the IsHidden() flag of the Actor because the APostProcessVolume is always Hidden by subclassing of ABrush
	return bEnabled != 0 && (bInGameWorld || (!bInGameWorld && bShowInEditor));
#else
	return bEnabled != 0;
#endif
}

bool APostProcessVolume::EncompassesPoint(FVector Point, float SphereRadius/*=0.f*/, float* OutDistanceToPoint)
{
	// Redirect IInterface_PostProcessVolume's non-const pure virtual EncompassesPoint virtual in to AVolume's non-virtual const EncompassesPoint
	return AVolume::EncompassesPoint(Point, SphereRadius, OutDistanceToPoint);
}

void APostProcessVolume::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);
#if WITH_EDITOR
	if (Ar.IsLoading())
	{
		Settings.OnAfterLoad();
	}
#endif
}

void APostProcessVolume::PostInitProperties()
{
	Super::PostInitProperties();

#if WITH_EDITOR
	if (!IsTemplate() && !VolumeGuid.IsValid())
	{
		VolumeGuid = FGuid::NewDeterministicGuid(GetPathName());
	}
#endif
}

#if WITH_EDITOR
void APostProcessVolume::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	static const FName NAME_Blendables = FName("Blendables");	

	const FName ChangedPropertyName = (PropertyChangedEvent.Property != NULL) ? PropertyChangedEvent.Property->GetFName() : NAME_None;	

	if(ChangedPropertyName == NAME_Blendables)
	{
		// remove unsupported types
		uint32 Count = Settings.WeightedBlendables.Array.Num();
		for(uint32 i = 0; i < Count; ++i)
		{
			UObject* Obj = Settings.WeightedBlendables.Array[i].Object;

			if(!Cast<IBlendableInterface>(Obj))
			{
				Settings.WeightedBlendables.Array[i] = FWeightedBlendable();
			}
		}
	}

	if (ChangedPropertyName == GET_MEMBER_NAME_CHECKED(APostProcessVolume, bUnbound))
	{
		if (bUnbound)
		{
			bIsSpatiallyLoaded = false;
		}
	}	
	
	if (PropertyChangedEvent.Property)
	{
#define CHECK_VIRTUALTEXTURE_USAGE(property)	{	\
													static const FName PropertyName = GET_MEMBER_NAME_STRING_CHECKED(FPostProcessSettings, property); \
													if (PropertyChangedEvent.Property->GetFName() == PropertyName)	\
													{	\
														VirtualTextureUtils::CheckAndReportInvalidUsage(this, PropertyName, Settings.property);	\
													}	\
												}
		
		CHECK_VIRTUALTEXTURE_USAGE(BloomDirtMask);
		CHECK_VIRTUALTEXTURE_USAGE(ColorGradingLUT);
		CHECK_VIRTUALTEXTURE_USAGE(LensFlareBokehShape);
#undef CHECK_VIRTUALTEXTURE_USAGE
	}
}

bool APostProcessVolume::CanEditChange(const FProperty* InProperty) const
{
	if (InProperty)
	{
		FString PropertyName = InProperty->GetName();

		// Settings, can be shared for multiple objects types (volume, component, camera, player)
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		{
			bool bIsMobile = false;

			if (UWorld* World = GetWorld())
			{
				if (FSceneInterface* Scene = World->Scene)
				{
					bIsMobile = Scene->GetShadingPath(Scene->GetFeatureLevel()) == EShadingPath::Mobile;
				}
			}

			bool bHaveCinematicDOF = !bIsMobile;
			bool bHaveGaussianDOF = bIsMobile;

			if (PropertyName == GET_MEMBER_NAME_STRING_CHECKED(FPostProcessSettings, DepthOfFieldScale) ||
				PropertyName == GET_MEMBER_NAME_STRING_CHECKED(FPostProcessSettings, DepthOfFieldNearBlurSize) ||
				PropertyName == GET_MEMBER_NAME_STRING_CHECKED(FPostProcessSettings, DepthOfFieldFarBlurSize) ||
				PropertyName == GET_MEMBER_NAME_STRING_CHECKED(FPostProcessSettings, DepthOfFieldSkyFocusDistance) ||
				PropertyName == GET_MEMBER_NAME_STRING_CHECKED(FPostProcessSettings, DepthOfFieldVignetteSize) ||
				PropertyName == GET_MEMBER_NAME_STRING_CHECKED(FPostProcessSettings, DepthOfFieldNearTransitionRegion) ||
				PropertyName == GET_MEMBER_NAME_STRING_CHECKED(FPostProcessSettings, DepthOfFieldFarTransitionRegion) ||
				PropertyName == GET_MEMBER_NAME_STRING_CHECKED(FPostProcessSettings, DepthOfFieldFocalRegion))
			{
				return bHaveGaussianDOF;
			}

			if (PropertyName == GET_MEMBER_NAME_STRING_CHECKED(FPostProcessSettings, DepthOfFieldDepthBlurAmount) ||
				PropertyName == GET_MEMBER_NAME_STRING_CHECKED(FPostProcessSettings, DepthOfFieldDepthBlurRadius) ||
				PropertyName == GET_MEMBER_NAME_STRING_CHECKED(FPostProcessSettings, DepthOfFieldMinFstop) ||
				PropertyName == GET_MEMBER_NAME_STRING_CHECKED(FPostProcessSettings, DepthOfFieldBladeCount))
			{
				return bHaveCinematicDOF;
			}

			if (PropertyName == GET_MEMBER_NAME_STRING_CHECKED(FPostProcessSettings, DepthOfFieldFstop))
			{
				return	( bHaveCinematicDOF || 
					      Settings.AutoExposureMethod == EAutoExposureMethod::AEM_Manual );
			}

			if (PropertyName == GET_MEMBER_NAME_STRING_CHECKED(FPostProcessSettings, CameraShutterSpeed) ||
				PropertyName == GET_MEMBER_NAME_STRING_CHECKED(FPostProcessSettings, CameraISO))
			{
				return Settings.AutoExposureMethod == EAutoExposureMethod::AEM_Manual;
			}

			// Parameters supported by both log-average and histogram Auto Exposure
			if (PropertyName == GET_MEMBER_NAME_STRING_CHECKED(FPostProcessSettings, AutoExposureMinBrightness) ||
				PropertyName == GET_MEMBER_NAME_STRING_CHECKED(FPostProcessSettings, AutoExposureMaxBrightness) ||
				PropertyName == GET_MEMBER_NAME_STRING_CHECKED(FPostProcessSettings, AutoExposureSpeedUp)       ||
				PropertyName == GET_MEMBER_NAME_STRING_CHECKED(FPostProcessSettings, AutoExposureSpeedDown))
			{
				return  ( Settings.AutoExposureMethod == EAutoExposureMethod::AEM_Histogram || 
					      Settings.AutoExposureMethod == EAutoExposureMethod::AEM_Basic );
			}

			// Parameters supported by only the histogram AutoExposure
			if (PropertyName == GET_MEMBER_NAME_STRING_CHECKED(FPostProcessSettings, AutoExposureLowPercent)  ||
				PropertyName == GET_MEMBER_NAME_STRING_CHECKED(FPostProcessSettings, AutoExposureHighPercent) ||
				PropertyName == GET_MEMBER_NAME_STRING_CHECKED(FPostProcessSettings, HistogramLogMin)         || 
				PropertyName == GET_MEMBER_NAME_STRING_CHECKED(FPostProcessSettings, HistogramLogMax) )
			{
				return Settings.AutoExposureMethod == EAutoExposureMethod::AEM_Histogram;
			}
			
			if (PropertyName == GET_MEMBER_NAME_STRING_CHECKED(FPostProcessSettings, LumenRayLightingMode))
			{
				static IConsoleVariable* RayTracingCVar = IConsoleManager::Get().FindConsoleVariable(TEXT("r.RayTracing"));
				if (RayTracingCVar->GetInt() == 0)
				{
					return false;
				}
			}

			if (PropertyName == GET_MEMBER_NAME_STRING_CHECKED(FPostProcessSettings, DynamicGlobalIlluminationMethod) ||
				PropertyName == GET_MEMBER_NAME_STRING_CHECKED(FPostProcessSettings, ReflectionMethod))
			{
				static IConsoleVariable* ForwardShadingCVar = IConsoleManager::Get().FindConsoleVariable(TEXT("r.ForwardShading"));
				if (ForwardShadingCVar->GetInt() != 0)
				{
					return false;
				}
			}
		}
		PRAGMA_ENABLE_DEPRECATION_WARNINGS

		if (PropertyName == GET_MEMBER_NAME_STRING_CHECKED(APostProcessVolume, bEnabled))
		{
			return true;
		}

		if (!bEnabled)
		{
			return false;
		}

		if (PropertyName == GET_MEMBER_NAME_STRING_CHECKED(APostProcessVolume, BlendRadius))
		{
			if (bUnbound)
			{
				return false;
			}
		}
	}

	return Super::CanEditChange(InProperty);
}

#endif // WITH_EDITOR

