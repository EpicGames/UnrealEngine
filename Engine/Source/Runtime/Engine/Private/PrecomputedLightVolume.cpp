// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	PrecomputedLightVolume.cpp: Implementation of a precomputed light volume.
=============================================================================*/

#include "PrecomputedLightVolume.h"
#include "SceneInterface.h"
#include "UObject/RenderingObjectVersion.h"
#include "PrimitiveDrawingUtils.h"
#include "Stats/StatsTrace.h"
#include "UnrealEngine.h"
#include "Engine/MapBuildDataRegistry.h"
#include "Interfaces/ITargetPlatform.h"

DECLARE_MEMORY_STAT(TEXT("Precomputed Light Volume"),STAT_PrecomputedLightVolumeBuildData,STATGROUP_MapBuildData);

template<> TVolumeLightingSample<2>::TVolumeLightingSample(const TVolumeLightingSample<2>& Other)
{
	Position = Other.Position;
	Radius = Other.Radius;
	Lighting = Other.Lighting;
	PackedSkyBentNormal = Other.PackedSkyBentNormal;
	DirectionalLightShadowing = Other.DirectionalLightShadowing;	
}
template<> TVolumeLightingSample<3>::TVolumeLightingSample(const TVolumeLightingSample<2>& Other)
{
	Position = Other.Position;
	Radius = Other.Radius;	
	PackedSkyBentNormal = Other.PackedSkyBentNormal;
	DirectionalLightShadowing = Other.DirectionalLightShadowing;

	for (int i = 0; i < 4; ++i)
	{
		Lighting.R.V[i] = Other.Lighting.R.V[i];
		Lighting.G.V[i] = Other.Lighting.G.V[i];
		Lighting.B.V[i] = Other.Lighting.B.V[i];
	}
	for (int i = 4; i < 9; ++i)
	{
		Lighting.R.V[i] = 0.0f;
		Lighting.G.V[i] = 0.0f;
		Lighting.B.V[i] = 0.0f;
	}
}

template<> TVolumeLightingSample<2>::TVolumeLightingSample(const TVolumeLightingSample<3>& Other)
{
	Position = Other.Position;
	Radius = Other.Radius;
	PackedSkyBentNormal = Other.PackedSkyBentNormal;
	DirectionalLightShadowing = Other.DirectionalLightShadowing;

	for (int i = 0; i < 4; ++i)
	{
		Lighting.R.V[i] = Other.Lighting.R.V[i];
		Lighting.G.V[i] = Other.Lighting.G.V[i];
		Lighting.B.V[i] = Other.Lighting.B.V[i];
	}
}

template<> TVolumeLightingSample<3>::TVolumeLightingSample(const TVolumeLightingSample<3>& Other)
{
	Position = Other.Position;
	Radius = Other.Radius;
	Lighting = Other.Lighting;
	PackedSkyBentNormal = Other.PackedSkyBentNormal;
	DirectionalLightShadowing = Other.DirectionalLightShadowing;
}


FArchive& operator<<(FArchive& Ar, TVolumeLightingSample<2>& Sample)
{
	Ar << Sample.Position;
	Ar << Sample.Radius;
	Ar << Sample.Lighting;

	if (Ar.UEVer() >= VER_UE4_SKY_BENT_NORMAL)
	{
		Ar << Sample.PackedSkyBentNormal;
	}

	if (Ar.UEVer() >= VER_UE4_VOLUME_SAMPLE_LOW_QUALITY_SUPPORT)
	{
		Ar << Sample.DirectionalLightShadowing;
	}
	
	return Ar;
}

FArchive& operator<<(FArchive& Ar, TVolumeLightingSample<3>& Sample)
{
	// Less version check since TVolumeLightingSample<3> require a more recent version. 
	Ar << Sample.Position;
	Ar << Sample.Radius;
	Ar << Sample.Lighting;
	Ar << Sample.PackedSkyBentNormal;
	Ar << Sample.DirectionalLightShadowing;
	return Ar;
}

FPrecomputedLightVolumeData::FPrecomputedLightVolumeData() :
	bInitialized(false),
	HighQualityLightmapOctree(FVector::ZeroVector, HALF_WORLD_MAX),
	LowQualityLightmapOctree(FVector::ZeroVector, HALF_WORLD_MAX)
{}

FPrecomputedLightVolumeData::~FPrecomputedLightVolumeData()
{
	if (bInitialized)
	{
		const SIZE_T VolumeBytes = GetAllocatedBytes();
		DEC_DWORD_STAT_BY(STAT_PrecomputedLightVolumeBuildData, VolumeBytes);
	}
}

static void LoadVolumeLightSamples(FArchive& Ar, int32 ArchiveNumSHSamples, TArray<FVolumeLightingSample>& Samples)
{
	// If it's the same number as what is currently compiled
	if (ArchiveNumSHSamples == NUM_INDIRECT_LIGHTING_SH_COEFFICIENTS) //-V517
	{
		Ar << Samples;		
	}
	else if (ArchiveNumSHSamples == 9)
	{
		TArray< TVolumeLightingSample<3> > DummySamples;
		Ar << DummySamples;
		for (TVolumeLightingSample<3>& LoadedSample : DummySamples)
		{
			Samples.Add(FVolumeLightingSample(LoadedSample));
		}
	}
	else
	{
		checkf(ArchiveNumSHSamples == 4, TEXT("NumSHSamples: %i"), ArchiveNumSHSamples);

		TArray< TVolumeLightingSample<2> > DummySamples;
		Ar << DummySamples;

		for (TVolumeLightingSample<2>& LoadedSample : DummySamples)
		{
			Samples.Add(FVolumeLightingSample(LoadedSample));
		}
	}	
}

FArchive& operator<<(FArchive& Ar,FPrecomputedLightVolumeData& Volume)
{
	Ar.UsingCustomVersion(FRenderingObjectVersion::GUID);

	if (Ar.IsCountingMemory())
	{
		const int32 AllocatedBytes = Volume.GetAllocatedBytes();
		Ar.CountBytes(AllocatedBytes, AllocatedBytes);
	}
	else if (Ar.IsLoading())
	{
		bool bVolumeInitialized = false;
		Ar << bVolumeInitialized; // Volume.bInitilized will be set in Volume.Initialize() call
		if (bVolumeInitialized)
		{
			FBox Bounds;
			Ar << Bounds;
			float SampleSpacing = 0.0f;
			Ar << SampleSpacing;
			Volume.Initialize(Bounds);

			// Before adding support for SH3, it was always using SH2
			int32 NumSHSamples = 4; 			
			if (Ar.CustomVer(FRenderingObjectVersion::GUID) >= FRenderingObjectVersion::IndirectLightingCache3BandSupport)
			{
				Ar << NumSHSamples;
			}

			TArray<FVolumeLightingSample> HighQualitySamples;
			LoadVolumeLightSamples(Ar, NumSHSamples, HighQualitySamples);

			// Deserialize samples as an array, and add them to the octree
			if (FPlatformProperties::SupportsHighQualityLightmaps() 
				&& (GIsEditor || AllowHighQualityLightmaps(GMaxRHIFeatureLevel)))
			{
				for(int32 SampleIndex = 0; SampleIndex < HighQualitySamples.Num(); SampleIndex++)
				{
					Volume.AddHighQualityLightingSample(HighQualitySamples[SampleIndex]);
				}
			}

			TArray<FVolumeLightingSample> LowQualitySamples;

			if (Ar.UEVer() >= VER_UE4_VOLUME_SAMPLE_LOW_QUALITY_SUPPORT)
			{
				LoadVolumeLightSamples(Ar, NumSHSamples, LowQualitySamples);
			}

			if (FPlatformProperties::SupportsLowQualityLightmaps() 
				&& (GIsEditor || !AllowHighQualityLightmaps(GMaxRHIFeatureLevel)))
			{
				for(int32 SampleIndex = 0; SampleIndex < LowQualitySamples.Num(); SampleIndex++)
				{
					Volume.AddLowQualityLightingSample(LowQualitySamples[SampleIndex]);
				}
			}

			Volume.FinalizeSamples();
		}
	}
	else if (Ar.IsSaving())
	{
		Ar << Volume.bInitialized;
		if (Volume.bInitialized)
		{
			Ar << Volume.Bounds;
			float SampleSpacing = 0.0f;
			Ar << SampleSpacing;

			int32 NumSHSamples = NUM_INDIRECT_LIGHTING_SH_COEFFICIENTS; 
			Ar << NumSHSamples;

			TArray<FVolumeLightingSample> HighQualitySamples;

			if (!Ar.IsCooking() || Ar.CookingTarget()->SupportsFeature(ETargetPlatformFeatures::HighQualityLightmaps))
			{
				// Gather an array of samples from the octree
				Volume.HighQualityLightmapOctree.FindAllElements([&HighQualitySamples](const FVolumeLightingSample& Sample)
				{
					HighQualitySamples.Add(Sample);
				});
			}
			
			Ar << HighQualitySamples;

			TArray<FVolumeLightingSample> LowQualitySamples;

			if (!Ar.IsCooking() || Ar.CookingTarget()->SupportsFeature(ETargetPlatformFeatures::LowQualityLightmaps))
			{
				// Gather an array of samples from the octree
				Volume.LowQualityLightmapOctree.FindAllElements([&LowQualitySamples](const FVolumeLightingSample& Sample)
				{
					LowQualitySamples.Add(Sample);
				});
			}

			Ar << LowQualitySamples;
		}
	}
	return Ar;
}

FArchive& operator<<(FArchive& Ar, FPrecomputedLightVolumeData*& Volume)
{
	bool bValid = Volume != NULL;

	Ar << bValid;

	if (bValid)
	{
		if (Ar.IsLoading())
		{
			Volume = new FPrecomputedLightVolumeData();
		}

		Ar << (*Volume);
	}

	return Ar;
}

/** Frees any previous samples, prepares the volume to have new samples added. */
void FPrecomputedLightVolumeData::Initialize(const FBox& NewBounds)
{
	InvalidateLightingCache();
	bInitialized = true;
	Bounds = NewBounds;
	// Initialize the octree based on the passed in bounds
	HighQualityLightmapOctree = FLightVolumeOctree(NewBounds.GetCenter(), NewBounds.GetExtent().GetMax());
	LowQualityLightmapOctree = FLightVolumeOctree(NewBounds.GetCenter(), NewBounds.GetExtent().GetMax());
}

/** Adds a lighting sample. */
void FPrecomputedLightVolumeData::AddHighQualityLightingSample(const FVolumeLightingSample& NewHighQualitySample)
{
	check(bInitialized);
	HighQualityLightmapOctree.AddElement(NewHighQualitySample);
}

void FPrecomputedLightVolumeData::AddLowQualityLightingSample(const FVolumeLightingSample& NewLowQualitySample)
{
	check(bInitialized);
	LowQualityLightmapOctree.AddElement(NewLowQualitySample);
}

/** Shrinks the octree and updates memory stats. */
void FPrecomputedLightVolumeData::FinalizeSamples()
{
	check(bInitialized);
	// No more samples will be added, shrink octree node element arrays
	HighQualityLightmapOctree.ShrinkElements();
	LowQualityLightmapOctree.ShrinkElements();
	const SIZE_T VolumeBytes = GetAllocatedBytes();
	INC_DWORD_STAT_BY(STAT_PrecomputedLightVolumeBuildData, VolumeBytes);
}

/** Invalidates anything produced by the last lighting build. */
void FPrecomputedLightVolumeData::InvalidateLightingCache()
{
	if (bInitialized)
	{
		// Release existing samples
		const SIZE_T VolumeBytes = GetAllocatedBytes();
		DEC_DWORD_STAT_BY(STAT_PrecomputedLightVolumeBuildData, VolumeBytes);
		HighQualityLightmapOctree.Destroy();
		LowQualityLightmapOctree.Destroy();
		bInitialized = false;
	}
}

SIZE_T FPrecomputedLightVolumeData::GetAllocatedBytes() const
{
	SIZE_T NodeBytes = 0;
	NodeBytes += HighQualityLightmapOctree.GetSizeBytes();
	NodeBytes += LowQualityLightmapOctree.GetSizeBytes();
	return NodeBytes;
}


FPrecomputedLightVolume::FPrecomputedLightVolume() :
	Data(NULL),
	bAddedToScene(false),
	OctreeForRendering(NULL),
	WorldOriginOffset(ForceInitToZero)
{}

FPrecomputedLightVolume::~FPrecomputedLightVolume()
{
}

void FPrecomputedLightVolume::AddToScene(FSceneInterface* Scene, UMapBuildDataRegistry* Registry, FGuid LevelBuildDataId)
{
	check(!bAddedToScene);

	const FPrecomputedLightVolumeData* NewData = NULL;

	if (Registry)
	{
		NewData = Registry->GetLevelPrecomputedLightVolumeBuildData(LevelBuildDataId);
	}

	if (NewData && NewData->bInitialized && Scene)
	{
		bAddedToScene = true;

		FPrecomputedLightVolume* Volume = this;

		ENQUEUE_RENDER_COMMAND(SetVolumeDataCommand)
			([Volume, NewData, Scene](FRHICommandListImmediate& RHICmdList) 
			{
				Volume->SetData(NewData, Scene);
			});
		Scene->AddPrecomputedLightVolume(this);
	}
}

void FPrecomputedLightVolume::RemoveFromScene(FSceneInterface* Scene)
{
	if (bAddedToScene)
	{
		bAddedToScene = false;

		if (Scene)
		{
			Scene->RemovePrecomputedLightVolume(this);
		}
	}

	WorldOriginOffset = FVector::ZeroVector;
}

void FPrecomputedLightVolume::SetData(const FPrecomputedLightVolumeData* NewData, FSceneInterface* Scene)
{
	Data = NewData;
	OctreeForRendering = AllowHighQualityLightmaps(Scene->GetFeatureLevel()) ? &Data->HighQualityLightmapOctree : &Data->LowQualityLightmapOctree;
}

void FPrecomputedLightVolume::InterpolateIncidentRadiancePoint(
		const FVector& InWorldPosition, 
		float& AccumulatedWeight,
		float& AccumulatedDirectionalLightShadowing,
		FSHVectorRGB3& AccumulatedIncidentRadiance,
		FVector& SkyBentNormal) const
{
	// This could be called on a NULL volume for a newly created level. This is now checked at the call site, but this check provides extra safety
	checkf(this, TEXT("FPrecomputedLightVolume::InterpolateIncidentRadiancePoint() is called on a null volume. Fix the call site."));

	// Handle being called on a volume that hasn't been initialized yet, which can happen if lighting hasn't been built yet.
	if (Data->bInitialized)
	{
		FVector WorldPosition = InWorldPosition - WorldOriginOffset; // relocate from world to volume space
		FBoxCenterAndExtent BoundingBox( WorldPosition, FVector::ZeroVector );
		// Iterate over the octree nodes containing the query point.
		OctreeForRendering->FindElementsWithBoundsTest(BoundingBox, [&AccumulatedIncidentRadiance, &SkyBentNormal, &AccumulatedDirectionalLightShadowing, &AccumulatedWeight, &WorldPosition](const FVolumeLightingSample& VolumeSample)
		{
			const float DistanceSquared = ((FVector)VolumeSample.Position - WorldPosition).SizeSquared();
			const float RadiusSquared = FMath::Square(VolumeSample.Radius);

			if (DistanceSquared < RadiusSquared)
			{
				const float InvRadiusSquared = 1.0f / RadiusSquared;
				// Weight each sample with the fraction that Position is to the center of the sample, and inversely to the sample radius.
				// The weight goes to 0 when Position is on the bounding radius of the sample, so the interpolated result is continuous.
				// The sample's radius size is a factor so that smaller samples contribute more than larger, low detail ones.
				const float SampleWeight = (1.0f - DistanceSquared * InvRadiusSquared) * InvRadiusSquared;
				// Accumulate weighted results and the total weight for normalization later
				AccumulatedIncidentRadiance += VolumeSample.Lighting * SampleWeight;
				SkyBentNormal += VolumeSample.GetSkyBentNormalUnpacked() * SampleWeight;
				AccumulatedDirectionalLightShadowing += VolumeSample.DirectionalLightShadowing * SampleWeight;
				AccumulatedWeight += SampleWeight;
			}
		});
	}
}

/** Interpolates incident radiance to Position. */
void FPrecomputedLightVolume::InterpolateIncidentRadianceBlock(
	const FBoxCenterAndExtent& InBoundingBox, 
	const FIntVector& QueryCellDimensions,
	const FIntVector& DestCellDimensions,
	const FIntVector& DestCellPosition,
	TArray<float>& AccumulatedWeights,
	TArray<FSHVectorRGB2>& AccumulatedIncidentRadiance) const
{	
	// This could be called on a NULL volume for a newly created level. This is now checked at the callsite, but this check provides extra safety
	checkf(this, TEXT("FPrecomputedLightVolume::InterpolateIncidentRadianceBlock() is called on a null volume. Fix the call site."));

	// Handle being called on a volume that hasn't been initialized yet, which can happen if lighting hasn't been built yet.
	if (Data->bInitialized)
	{
		FBoxCenterAndExtent BoundingBox = InBoundingBox;
		BoundingBox.Center = BoundingBox.Center - FVector4(WorldOriginOffset, 0.f); // relocate from world to volume space
		
		static TArray<const FVolumeLightingSample*> PotentiallyIntersectingSamples;
		PotentiallyIntersectingSamples.Reset(100);

		// Iterate over the octree nodes containing the query point.
		OctreeForRendering->FindElementsWithBoundsTest(BoundingBox, [&](const FVolumeLightingSample& VolumeSample)
		{
			PotentiallyIntersectingSamples.Add(&VolumeSample);
		});
		
		const int32 LinearIndexBase = DestCellPosition.Z * DestCellDimensions.Y * DestCellDimensions.X
			+ DestCellPosition.Y * DestCellDimensions.X
			+ DestCellPosition.X;

		const int32 CenterIndex = LinearIndexBase + (QueryCellDimensions.Z / 2 * DestCellDimensions.Y + QueryCellDimensions.Y / 2) * DestCellDimensions.X + QueryCellDimensions.X / 2;

		for (int32 SampleIndex = 0; SampleIndex < PotentiallyIntersectingSamples.Num(); SampleIndex++)
		{
			const FVolumeLightingSample& VolumeSample = *PotentiallyIntersectingSamples[SampleIndex];
				
			const float RadiusSquared = FMath::Square(VolumeSample.Radius);
			const float WeightBase  = 1.0f / RadiusSquared;
			const float WeightMultiplier = -1.0f / (RadiusSquared * RadiusSquared);
				
			const FVector BaseTranslationFromSample = BoundingBox.Center - BoundingBox.Extent - FVector4((FVector)VolumeSample.Position);
			const FVector QuerySteps = BoundingBox.Extent / FVector(QueryCellDimensions) * 2;
			FVector TranslationFromSample = BaseTranslationFromSample;

			for (int32 Z = 0; Z < QueryCellDimensions.Z; Z++)
			{
				for (int32 Y = 0; Y < QueryCellDimensions.Y; Y++)
				{
					for (int32 X = 0; X < QueryCellDimensions.X; X++)
					{
						const float DistanceSquared = TranslationFromSample.SizeSquared();

						if (DistanceSquared < RadiusSquared)
						{
							const int32 LinearIndex = LinearIndexBase + (Z * DestCellDimensions.Y + Y) * DestCellDimensions.X + X;
							// Weight each sample with the fraction that Position is to the center of the sample, and inversely to the sample radius.
							// The weight goes to 0 when Position is on the bounding radius of the sample, so the interpolated result is continuous.
							// The sample's radius size is a factor so that smaller samples contribute more than larger, low detail ones.
							const float SampleWeight = DistanceSquared * WeightMultiplier + WeightBase;
							// Accumulate weighted results and the total weight for normalization later													
							AccumulatedIncidentRadiance[LinearIndex] += TSHVectorRGB<2>(VolumeSample.Lighting) * SampleWeight;
							AccumulatedWeights[LinearIndex] += SampleWeight;							
						}

						TranslationFromSample.X += QuerySteps.X;
					}

					TranslationFromSample.X = BaseTranslationFromSample.X;
					TranslationFromSample.Y += QuerySteps.Y;
				}

				TranslationFromSample.Y = BaseTranslationFromSample.Y;
				TranslationFromSample.Z += QuerySteps.Z;
			}
		}
	}
}

void FPrecomputedLightVolume::DebugDrawSamples(FPrimitiveDrawInterface* PDI, bool bDrawDirectionalShadowing) const
{
	OctreeForRendering->FindAllElements([bDrawDirectionalShadowing, PDI, this](const FVolumeLightingSample& VolumeSample)
	{
		const FLinearColor AverageColor = bDrawDirectionalShadowing
			? FLinearColor(VolumeSample.DirectionalLightShadowing, VolumeSample.DirectionalLightShadowing, VolumeSample.DirectionalLightShadowing)
			: VolumeSample.Lighting.CalcIntegral() / (FSHVector2::ConstantBasisIntegral * UE_PI);
		
		FVector SamplePosition = (FVector)VolumeSample.Position + WorldOriginOffset; //relocate from volume to world space
		PDI->DrawPoint(SamplePosition, AverageColor, 10, SDPG_World);
	});
}

bool FPrecomputedLightVolume::IntersectBounds(const FBoxSphereBounds& InBounds) const
{
	if (OctreeForRendering)
	{
		FBox VolumeBounds = OctreeForRendering->GetRootBounds().GetBox();
		return InBounds.GetBox().Intersect(VolumeBounds);
	}

	return false;
}

void FPrecomputedLightVolume::ApplyWorldOffset(const FVector& InOffset)
{
	WorldOriginOffset+= InOffset;
}
