// Copyright Epic Games, Inc. All Rights Reserved.

#include "Lightmass/PrecomputedVisibilityVolume.h"
#include "Async/TaskGraphInterfaces.h"
#include "Engine/CollisionProfile.h"
#include "Components/BrushComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PrecomputedVisibilityVolume)

APrecomputedVisibilityVolume::APrecomputedVisibilityVolume(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	GetBrushComponent()->SetCollisionProfileName(UCollisionProfile::NoCollision_ProfileName);
	bColored = true;
	BrushColor.R = 25;
	BrushColor.G = 255;
	BrushColor.B = 25;
	BrushColor.A = 255;

}

