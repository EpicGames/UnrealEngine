// Copyright Epic Games, Inc. All Rights Reserved.

#include "Async/TaskGraphInterfaces.h"
#include "Engine/CollisionProfile.h"
#include "Components/BrushComponent.h"
#include "GameFramework/Volume.h"
#include "Lightmass/PrecomputedVisibilityOverrideVolume.h"

APrecomputedVisibilityOverrideVolume::APrecomputedVisibilityOverrideVolume(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	GetBrushComponent()->SetCollisionProfileName(UCollisionProfile::NoCollision_ProfileName);

	bColored = true;
	BrushColor.R = 25;
	BrushColor.G = 120;
	BrushColor.B = 90;
	BrushColor.A = 255;

}
