// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	PrecomputedVolumetricLightmap.cpp
=============================================================================*/

#include "PrecomputedVolumetricLightmap.h"
#include "Engine/Texture.h"
#include "UObject/UE5MainStreamObjectVersion.h"
#include "Engine/MapBuildDataRegistry.h"
#include "Engine/World.h"
#include "UObject/MobileObjectVersion.h"
#include "RenderGraphUtils.h"
// FIXME: temp fix for ordering issue between WorldContext.World()->InitWorld(); and GShaderCompilingManager->ProcessAsyncResults(false, true); in UnrealEngine.cpp
#include "RenderingThread.h"
#include "ShaderCompiler.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "SceneInterface.h"
#include "Stats/StatsTrace.h"
#include "ProfilingDebugging/RealtimeGPUProfiler.h"
#include "RenderUtils.h"

DECLARE_MEMORY_STAT(TEXT("Volumetric Lightmap"),STAT_VolumetricLightmapBuildData,STATGROUP_MapBuildData);

const static FLazyName VolumetricLightmapDataLayerName(TEXT("VolumetricLightmapDataLayer"));

void FVolumetricLightmapDataLayer::CreateTexture(FRHICommandListBase& RHICmdList, FIntVector Dimensions)
{
	const FRHITextureCreateDesc Desc =
		FRHITextureCreateDesc::Create3D(TEXT("VolumetricLightmap"), Dimensions, Format)
		.SetFlags(ETextureCreateFlags::ShaderResource | ETextureCreateFlags::UAV)
		.SetInitActionBulkData(this)
		.SetClassName(VolumetricLightmapDataLayerName);

	Texture = RHICmdList.CreateTexture(Desc);
}

void FVolumetricLightmapDataLayer::CreateTargetTexture(FRHICommandListBase& RHICmdList, FIntVector Dimensions)
{
	const FRHITextureCreateDesc Desc =
		FRHITextureCreateDesc::Create3D(TEXT("VolumetricLightmap"), Dimensions, Format)
		.SetFlags(ETextureCreateFlags::ShaderResource | ETextureCreateFlags::UAV)
		.SetClassName(VolumetricLightmapDataLayerName);

	Texture = RHICmdList.CreateTexture(Desc);
}

void FVolumetricLightmapDataLayer::CreateTexture(FIntVector Dimensions)
{
	CreateTexture(FRHICommandListImmediate::Get(), Dimensions);
}

void FVolumetricLightmapDataLayer::CreateTargetTexture(FIntVector Dimensions)
{
	CreateTargetTexture(FRHICommandListImmediate::Get(), Dimensions);
}

void FVolumetricLightmapDataLayer::CreateUAV(FRHICommandListBase& RHICmdList)
{
	check(Texture);

	UAV = RHICmdList.CreateUnorderedAccessView(Texture, FRHIViewDesc::CreateTextureUAV().SetDimensionFromTexture(Texture));
}

TGlobalResource<FVolumetricLightmapBrickAtlas> GVolumetricLightmapBrickAtlas;

inline void ConvertBGRA8ToRGBA8ForLayer(FVolumetricLightmapDataLayer& Layer)
{
	if (Layer.Format == PF_B8G8R8A8)
	{
		for (int32 PixelIndex = 0; PixelIndex < Layer.Data.Num() / GPixelFormats[PF_B8G8R8A8].BlockBytes; PixelIndex++)
		{
			FColor Color;

			Color.B = Layer.Data[PixelIndex * 4 + 0];
			Color.G = Layer.Data[PixelIndex * 4 + 1];
			Color.R = Layer.Data[PixelIndex * 4 + 2];
			Color.A = Layer.Data[PixelIndex * 4 + 3];

			Layer.Data[PixelIndex * 4 + 0] = Color.R;
			Layer.Data[PixelIndex * 4 + 1] = Color.G;
			Layer.Data[PixelIndex * 4 + 2] = Color.B;
			Layer.Data[PixelIndex * 4 + 3] = Color.A;
		}

		Layer.Format = PF_R8G8B8A8;
	}
}

FArchive& operator<<(FArchive& Ar,FVolumetricLightmapDataLayer& Layer)
{
	Ar << Layer.Data;
	
	if (Ar.IsLoading())
	{
		Layer.DataSize = Layer.Data.Num() * Layer.Data.GetTypeSize();
	}

	UEnum* PixelFormatEnum = UTexture::GetPixelFormatEnum();

	if (Ar.IsLoading())
	{
		FString PixelFormatString;
		Ar << PixelFormatString;
		Layer.Format = (EPixelFormat)PixelFormatEnum->GetValueByName(*PixelFormatString);

		ConvertBGRA8ToRGBA8ForLayer(Layer);
	}
	else if (Ar.IsSaving())
	{
		FString PixelFormatString = PixelFormatEnum->GetNameByValue(Layer.Format).GetPlainNameString();
		Ar << PixelFormatString;
	}

	return Ar;
}

FArchive& operator<<(FArchive& Ar,FPrecomputedVolumetricLightmapData& Volume)
{
	Ar.UsingCustomVersion(FMobileObjectVersion::GUID);
	Ar.UsingCustomVersion(FRenderingObjectVersion::GUID);
	Ar.UsingCustomVersion(FUE5MainStreamObjectVersion::GUID);

	Ar << Volume.Bounds;
	Ar << Volume.IndirectionTextureDimensions;
	Ar << Volume.IndirectionTexture;

	Ar << Volume.BrickSize;
	Ar << Volume.BrickDataDimensions;

	Ar << Volume.BrickData.AmbientVector;

	for (int32 i = 0; i < UE_ARRAY_COUNT(Volume.BrickData.SHCoefficients); i++)
	{
		Ar << Volume.BrickData.SHCoefficients[i];
	}

	Ar << Volume.BrickData.SkyBentNormal;
	Ar << Volume.BrickData.DirectionalLightShadowing;
	
	if (Ar.CustomVer(FMobileObjectVersion::GUID) >= FMobileObjectVersion::LQVolumetricLightmapLayers)
	{
		if (Ar.CustomVer(FUE5MainStreamObjectVersion::GUID) < FUE5MainStreamObjectVersion::MobileStationaryLocalLights && Ar.IsLoading())
		{
			// Don't serialize cooked LQ data
			FVolumetricLightmapDataLayer Dummy;
			Ar << Dummy;
			Ar << Dummy;
		}
	}

	if (Ar.CustomVer(FRenderingObjectVersion::GUID) >= FRenderingObjectVersion::VolumetricLightmapStreaming)
	{
		Ar << Volume.SubLevelBrickPositions;
		Ar << Volume.IndirectionTextureOriginalValues;
	}

	if (Ar.IsLoading())
	{
		Volume.bTransient = false;

		const SIZE_T VolumeBytes = Volume.GetAllocatedBytes();
		INC_DWORD_STAT_BY(STAT_VolumetricLightmapBuildData, VolumeBytes);
	}

	return Ar;
}

FArchive& operator<<(FArchive& Ar, FPrecomputedVolumetricLightmapData*& Volume)
{
	bool bValid = Volume != NULL;

	Ar << bValid;

	if (bValid)
	{
		if (Ar.IsLoading())
		{
			Volume = new FPrecomputedVolumetricLightmapData();
		}

		Ar << (*Volume);
	}

	return Ar;
}

int32 FVolumetricLightmapBrickData::GetMinimumVoxelSize() const
{
	check(AmbientVector.Format != PF_Unknown);
	int32 VoxelSize = GPixelFormats[AmbientVector.Format].BlockBytes;

	for (int32 i = 0; i < UE_ARRAY_COUNT(SHCoefficients); i++)
	{
		VoxelSize += GPixelFormats[SHCoefficients[i].Format].BlockBytes;
	}
		
	// excluding SkyBentNormal because it is conditional

	VoxelSize += GPixelFormats[DirectionalLightShadowing.Format].BlockBytes;

	return VoxelSize;
}

FPrecomputedVolumetricLightmapData::FPrecomputedVolumetricLightmapData()
	: Bounds(EForceInit::ForceInitToZero)
	, bTransient(true)
	, IndirectionTextureDimensions(EForceInit::ForceInitToZero)
	, BrickSize(0)
	, BrickDataDimensions(EForceInit::ForceInitToZero)
	, BrickDataBaseOffsetInAtlas(0)
{}

FPrecomputedVolumetricLightmapData::~FPrecomputedVolumetricLightmapData()
{
	if (!bTransient)
	{
		const SIZE_T VolumeBytes = GetAllocatedBytes();
		DEC_DWORD_STAT_BY(STAT_VolumetricLightmapBuildData, VolumeBytes);
	}
}

/** */
void FPrecomputedVolumetricLightmapData::InitializeOnImport(const FBox& NewBounds, int32 InBrickSize)
{
	Bounds = NewBounds;
	BrickSize = InBrickSize;
}

void FPrecomputedVolumetricLightmapData::FinalizeImport()
{
	bTransient = false;
	const SIZE_T VolumeBytes = GetAllocatedBytes();
	INC_DWORD_STAT_BY(STAT_VolumetricLightmapBuildData, VolumeBytes);
}

ENGINE_API void FPrecomputedVolumetricLightmapData::InitRHI(FRHICommandListBase& RHICmdList)
{
	if (GetFeatureLevel() >= ERHIFeatureLevel::SM5)
	{
		if (IndirectionTextureDimensions.GetMax() > 0)
		{
			IndirectionTexture.CreateTexture(RHICmdList, IndirectionTextureDimensions);
		}

		if (BrickDataDimensions.GetMax() > 0)
		{
			BrickData.AmbientVector.CreateTexture(RHICmdList, BrickDataDimensions);

			for (int32 i = 0; i < UE_ARRAY_COUNT(BrickData.SHCoefficients); i++)
			{
				BrickData.SHCoefficients[i].CreateTexture(RHICmdList, BrickDataDimensions);
			}

			if (BrickData.SkyBentNormal.Data.Num() > 0)
			{
				BrickData.SkyBentNormal.CreateTexture(RHICmdList, BrickDataDimensions);
			}

			BrickData.DirectionalLightShadowing.CreateTexture(RHICmdList, BrickDataDimensions);
		}

		GVolumetricLightmapBrickAtlas.Insert(RHICmdList.GetAsImmediate(), INT_MAX, this);

		// It is now safe to release the brick data used for upload. They will stay in GPU memory until UMapBuildDataRegistry::BeginDestroy().
		BrickData.ReleaseRHI();
	}
}

void FPrecomputedVolumetricLightmapData::InitRHIForSubLevelResources(FRHICommandListBase& RHICmdList)
{
	if (SubLevelBrickPositions.Num() > 0)
	{
		SubLevelBrickPositions.SetAllowCPUAccess(true);
		IndirectionTextureOriginalValues.SetAllowCPUAccess(true);

		{
			const FRHIBufferCreateDesc CreateDesc =
				FRHIBufferCreateDesc::CreateVertex(TEXT("SubLevelBrickPositionsBuffer"), SubLevelBrickPositions.Num() * SubLevelBrickPositions.GetTypeSize())
				.AddUsage(BUF_Static | BUF_ShaderResource)
				.SetInitActionResourceArray(&SubLevelBrickPositions)
				.DetermineInitialState();

			SubLevelBrickPositionsBuffer = RHICmdList.CreateBuffer(CreateDesc);
			SubLevelBrickPositionsSRV = RHICmdList.CreateShaderResourceView(
				SubLevelBrickPositionsBuffer, 
				FRHIViewDesc::CreateBufferSRV()
					.SetType(FRHIViewDesc::EBufferType::Typed)
					.SetFormat(PF_R32_UINT));
		}

		{
			const FRHIBufferCreateDesc CreateDesc =
				FRHIBufferCreateDesc::CreateVertex(TEXT("IndirectionTextureOriginalValuesBuffer"), IndirectionTextureOriginalValues.Num() * IndirectionTextureOriginalValues.GetTypeSize())
				.AddUsage(BUF_Static | BUF_ShaderResource)
				.SetInitActionResourceArray(&IndirectionTextureOriginalValues)
				.DetermineInitialState();

			IndirectionTextureOriginalValuesBuffer = RHICmdList.CreateBuffer(CreateDesc);
			IndirectionTextureOriginalValuesSRV = RHICmdList.CreateShaderResourceView(
				IndirectionTextureOriginalValuesBuffer, 
				FRHIViewDesc::CreateBufferSRV()
					.SetType(FRHIViewDesc::EBufferType::Typed)
					.SetFormat(PF_R8G8B8A8_UINT));
		}
	}
}

void FPrecomputedVolumetricLightmapData::InitRHIForSubLevelResources()
{
	InitRHIForSubLevelResources(FRHICommandListImmediate::Get());
}

ENGINE_API void FPrecomputedVolumetricLightmapData::ReleaseRHI()
{
	if (GetFeatureLevel() >= ERHIFeatureLevel::SM5)
	{
		GVolumetricLightmapBrickAtlas.Remove(FRHICommandListImmediate::Get(), this);

		{
			IndirectionTexture.Texture.SafeRelease();
			IndirectionTexture.UAV.SafeRelease();
			BrickData.ReleaseRHI();
		}
	}
}

ENGINE_API void FPrecomputedVolumetricLightmapData::ReleaseRHIForSubLevelResources()
{
	{
		SubLevelBrickPositionsBuffer.SafeRelease();
		SubLevelBrickPositionsSRV.SafeRelease();

		IndirectionTextureOriginalValuesBuffer.SafeRelease();
		IndirectionTextureOriginalValuesSRV.SafeRelease();
	}
}

ENGINE_API void FPrecomputedVolumetricLightmapData::HandleDataMovementInAtlas(FRHIComputeCommandList& RHICmdList, int32 OldOffset, int32 NewOffset)
{
	BrickDataBaseOffsetInAtlas = NewOffset;

	if (IndirectionTextureDimensions.GetMax() > 0)
	{
		const int32 PaddedBrickSize = BrickSize + 1;
		int32 NumBricks = BrickDataDimensions.X * BrickDataDimensions.Y * BrickDataDimensions.Z / (PaddedBrickSize * PaddedBrickSize * PaddedBrickSize);

		for (FPrecomputedVolumetricLightmapData* SceneData : SceneDataAdded)
		{
			if (SceneData->IndirectionTexture.Texture.IsValid())
			{
				FGlobalShaderMap* GlobalShaderMap = GetGlobalShaderMap(GetFeatureLevel());

				TShaderMapRef<FMoveWholeIndirectionTextureCS> ComputeShader(GlobalShaderMap);

				FVolumetricLightmapDataLayer NewIndirectionTexture = SceneData->IndirectionTexture;
				NewIndirectionTexture.CreateTargetTexture(RHICmdList, IndirectionTextureDimensions);
				NewIndirectionTexture.CreateUAV(RHICmdList);

				FMoveWholeIndirectionTextureCS::FParameters Parameters;
				Parameters.NumBricks = NumBricks;
				Parameters.StartPosInOldVolume = OldOffset;
				Parameters.StartPosInNewVolume = BrickDataBaseOffsetInAtlas;
				Parameters.OldIndirectionTexture = SceneData->IndirectionTexture.Texture;
				Parameters.IndirectionTexture = NewIndirectionTexture.UAV;

				FComputeShaderUtils::Dispatch(RHICmdList, ComputeShader, Parameters,
					FIntVector(
						FMath::DivideAndRoundUp(IndirectionTextureDimensions.X, 4),
						FMath::DivideAndRoundUp(IndirectionTextureDimensions.Y, 4),
						FMath::DivideAndRoundUp(IndirectionTextureDimensions.Z, 4))
				);

				SceneData->IndirectionTexture = NewIndirectionTexture;

				FRHIUnorderedAccessView* UAV = NewIndirectionTexture.UAV;
				RHICmdList.Transition(FRHITransitionInfo(UAV, ERHIAccess::Unknown, ERHIAccess::SRVMask));
			}
		}
	}
	else
	{
		InitRHIForSubLevelResources(RHICmdList);

		for (FPrecomputedVolumetricLightmapData* SceneData : SceneDataAdded)
		{
			if (SceneData->IndirectionTexture.Texture.IsValid())
			{
				FGlobalShaderMap* GlobalShaderMap = GetGlobalShaderMap(GetFeatureLevel());

				TShaderMapRef<FPatchIndirectionTextureCS> ComputeShader(GlobalShaderMap);

				int32 NumBricks = SubLevelBrickPositions.Num();

				FPatchIndirectionTextureCS::FParameters Parameters;
				Parameters.NumBricks = NumBricks;
				Parameters.StartPosInNewVolume = BrickDataBaseOffsetInAtlas;
				Parameters.IndirectionTexture = SceneData->IndirectionTexture.UAV;
				Parameters.SubLevelBrickPositions = SubLevelBrickPositionsSRV;

				FComputeShaderUtils::Dispatch(RHICmdList, ComputeShader, Parameters, FIntVector(FMath::DivideAndRoundUp(NumBricks, 64), 1, 1));

				FRHIUnorderedAccessView* UAV = SceneData->IndirectionTexture.UAV;
				RHICmdList.Transition(FRHITransitionInfo(UAV, ERHIAccess::Unknown, ERHIAccess::SRVMask));
			}
		}

		ReleaseRHIForSubLevelResources();
	}
}

inline FIntVector ComputeBrickLayoutPosition(int32 BrickLayoutAllocation, FIntVector BrickLayoutDimensions)
{
	const FIntVector BrickPosition(
		BrickLayoutAllocation % BrickLayoutDimensions.X,
		(BrickLayoutAllocation / BrickLayoutDimensions.X) % BrickLayoutDimensions.Y,
		BrickLayoutAllocation / (BrickLayoutDimensions.X * BrickLayoutDimensions.Y));

	return BrickPosition;
}

ENGINE_API void FPrecomputedVolumetricLightmapData::AddToSceneData(FRHICommandListImmediate& RHICmdList, FPrecomputedVolumetricLightmapData* SceneData)
{
	if (SceneDataAdded.Find(SceneData) != INDEX_NONE)
	{
		return;
	}

	if (IndirectionTextureDimensions.GetMax() > 0)
	{
		SceneDataAdded.Add(SceneData);

		// Copy parameters from the persistent level VLM
		SceneData->Bounds = Bounds;
		SceneData->BrickSize = BrickSize;
		SceneData->BrickDataDimensions = BrickDataDimensions;

		SceneData->IndirectionTexture.Format = IndirectionTexture.Format;
		SceneData->IndirectionTextureDimensions = IndirectionTextureDimensions;

		if (GetFeatureLevel() >= ERHIFeatureLevel::SM5)
		{
			// Tracking down UE-88573
			ensureMsgf(IsInitialized(), TEXT("FPrecomputedVolumetricLightmapData is being added to a scene data without initialization"));

			if (!IsInitialized())
			{
				InitResource(RHICmdList);
			}

			if (!IndirectionTexture.Texture)
		{
				ensureMsgf(IsInitialized(), TEXT("FPrecomputedVolumetricLightmapData IndirectionTexture is still invalid after manual initialization, returning"));
				return;
			}

			// GPU Path

			const int32 PaddedBrickSize = BrickSize + 1;
			int32 NumBricks = BrickDataDimensions.X * BrickDataDimensions.Y * BrickDataDimensions.Z / (PaddedBrickSize * PaddedBrickSize * PaddedBrickSize);

			FGlobalShaderMap* GlobalShaderMap = GetGlobalShaderMap(GetFeatureLevel());

			TShaderMapRef<FMoveWholeIndirectionTextureCS> ComputeShader(GlobalShaderMap);

			FVolumetricLightmapDataLayer NewIndirectionTexture = SceneData->IndirectionTexture;
			NewIndirectionTexture.CreateTargetTexture(RHICmdList, IndirectionTextureDimensions);
			NewIndirectionTexture.CreateUAV(RHICmdList);

			RHICmdList.Transition(FRHITransitionInfo(NewIndirectionTexture.UAV, ERHIAccess::Unknown, ERHIAccess::UAVCompute));

			FMoveWholeIndirectionTextureCS::FParameters Parameters;
			Parameters.NumBricks = NumBricks;
			Parameters.StartPosInOldVolume = 0;
			Parameters.StartPosInNewVolume = BrickDataBaseOffsetInAtlas;
			Parameters.OldIndirectionTexture = IndirectionTexture.Texture;
			Parameters.IndirectionTexture = NewIndirectionTexture.UAV;

			FComputeShaderUtils::Dispatch(RHICmdList, ComputeShader, Parameters,
				FIntVector(
					FMath::DivideAndRoundUp(IndirectionTextureDimensions.X, 4),
					FMath::DivideAndRoundUp(IndirectionTextureDimensions.Y, 4),
					FMath::DivideAndRoundUp(IndirectionTextureDimensions.Z, 4))
			);

			RHICmdList.Transition(FRHITransitionInfo(NewIndirectionTexture.UAV, ERHIAccess::UAVCompute, ERHIAccess::SRVMask));

			SceneData->IndirectionTexture = NewIndirectionTexture;

			if (!GIsEditor)
			{
				// Steal the indirection texture. When the sublevels are unloaded the values will be restored.
				IndirectionTexture = SceneData->IndirectionTexture;			
			}

			RHICmdList.Transition(FRHITransitionInfo(IndirectionTexture.Texture, ERHIAccess::Unknown, ERHIAccess::SRVMask));
			RHICmdList.Transition(FRHITransitionInfo(NewIndirectionTexture.UAV, ERHIAccess::Unknown, ERHIAccess::SRVMask));
		}
		else
		{
			// CPU Path
			SceneData->IndirectionTexture.Data = IndirectionTexture.Data;
			SceneData->CPUSubLevelIndirectionTable.Empty();
			SceneData->CPUSubLevelIndirectionTable.AddZeroed(IndirectionTextureDimensions.X * IndirectionTextureDimensions.Y * IndirectionTextureDimensions.Z);
			SceneData->CPUSubLevelBrickDataList.Empty();
			SceneData->CPUSubLevelBrickDataList.Add(this);
		}
	}
	else
	{
		if (GetFeatureLevel() >= ERHIFeatureLevel::SM5)
		{
			// GPU Path
			if (SceneData->IndirectionTexture.Texture.IsValid())
			{
				SceneDataAdded.Add(SceneData);

				InitRHIForSubLevelResources(RHICmdList);

				FGlobalShaderMap* GlobalShaderMap = GetGlobalShaderMap(GetFeatureLevel());

				TShaderMapRef<FPatchIndirectionTextureCS> ComputeShader(GlobalShaderMap);

				int32 NumBricks = SubLevelBrickPositions.Num();

				FPatchIndirectionTextureCS::FParameters Parameters;
				Parameters.NumBricks = NumBricks;
				Parameters.StartPosInNewVolume = BrickDataBaseOffsetInAtlas;
				Parameters.IndirectionTexture = SceneData->IndirectionTexture.UAV;
				Parameters.SubLevelBrickPositions = SubLevelBrickPositionsSRV;

				RHICmdList.Transition(FRHITransitionInfo(SceneData->IndirectionTexture.UAV, ERHIAccess::Unknown, ERHIAccess::UAVCompute));

				FComputeShaderUtils::Dispatch(RHICmdList, ComputeShader, Parameters, FIntVector(FMath::DivideAndRoundUp(NumBricks, 64), 1, 1));

				RHICmdList.Transition(FRHITransitionInfo(SceneData->IndirectionTexture.UAV, ERHIAccess::UAVCompute, ERHIAccess::SRVMask));

				ReleaseRHIForSubLevelResources();
			}
		}
		else
		{
			// CPU Path
			if (SceneData->IndirectionTexture.Data.Num() > 0)
			{
				SceneDataAdded.Add(SceneData);
				// Find empty spot or Add new
				int32 IndexInCPUSubLevelBrickDataList = SceneData->CPUSubLevelBrickDataList.Find(nullptr);
				if (IndexInCPUSubLevelBrickDataList == INDEX_NONE)
				{
					IndexInCPUSubLevelBrickDataList = SceneData->CPUSubLevelBrickDataList.Add(nullptr);
				}
				check(IndexInCPUSubLevelBrickDataList < UINT8_MAX);
				uint8 Value = (uint8)IndexInCPUSubLevelBrickDataList;
				SceneData->CPUSubLevelBrickDataList[IndexInCPUSubLevelBrickDataList] = this;

				const int32 PaddedBrickSize = BrickSize + 1;
				const FIntVector BrickLayoutDimensions = BrickDataDimensions / PaddedBrickSize;

				for (int32 BrickIndex = 0; BrickIndex < SubLevelBrickPositions.Num(); BrickIndex++)
				{
					const FIntVector BrickLayoutPosition = ComputeBrickLayoutPosition(BrickIndex, BrickLayoutDimensions);

					const FIntVector IndirectionDestDataCoordinate = SubLevelBrickPositions[BrickIndex];
					const int32 IndirectionDestDataIndex =
						((IndirectionDestDataCoordinate.Z * SceneData->IndirectionTextureDimensions.Y) + IndirectionDestDataCoordinate.Y) *
						SceneData->IndirectionTextureDimensions.X + IndirectionDestDataCoordinate.X;

					{
						const int32 IndirectionTextureDataStride = GPixelFormats[SceneData->IndirectionTexture.Format].BlockBytes;
						uint8* IndirectionVoxelPtr = (uint8*)&SceneData->IndirectionTexture.Data[IndirectionDestDataIndex * IndirectionTextureDataStride];
						*(IndirectionVoxelPtr + 0) = BrickLayoutPosition.X;
						*(IndirectionVoxelPtr + 1) = BrickLayoutPosition.Y;
						*(IndirectionVoxelPtr + 2) = BrickLayoutPosition.Z;
						*(IndirectionVoxelPtr + 3) = 1;
					}

					{
						SceneData->CPUSubLevelIndirectionTable[IndirectionDestDataIndex] = Value;
					}
				}
			}
		}
	}
}

ENGINE_API void FPrecomputedVolumetricLightmapData::RemoveFromSceneData(FRHICommandListImmediate& RHICmdList, FPrecomputedVolumetricLightmapData* SceneData, int32 PersistentLevelBrickDataBaseOffset)
{
	if (SceneDataAdded.Find(SceneData) == INDEX_NONE)
	{
		return;
	}

	SceneDataAdded.Remove(SceneData);

	if (IndirectionTextureDimensions.GetMax() > 0)
	{
		// Do nothing, as when a VLM data with indirection texture is being destroyed, the persistent level is going away
	}
	else
	{
		if (GetFeatureLevel() >= ERHIFeatureLevel::SM5)
		{
			if (SceneData->IndirectionTexture.Texture.IsValid())
			{
				SCOPED_DRAW_EVENT(RHICmdList, RemoveSubLevelBricksCS);

				InitRHIForSubLevelResources(RHICmdList);

				FGlobalShaderMap* GlobalShaderMap = GetGlobalShaderMap(GetFeatureLevel());

				TShaderMapRef<FRemoveSubLevelBricksCS> ComputeShader(GlobalShaderMap);

				FRemoveSubLevelBricksCS::FParameters Parameters;
				Parameters.NumBricks = SubLevelBrickPositions.Num();
				Parameters.IndirectionTexture = SceneData->IndirectionTexture.UAV;
				Parameters.SubLevelBrickPositions = SubLevelBrickPositionsSRV;
				Parameters.IndirectionTextureOriginalValues = IndirectionTextureOriginalValuesSRV;
				Parameters.PersistentLevelBrickDataBaseOffset = PersistentLevelBrickDataBaseOffset;

				RHICmdList.Transition(FRHITransitionInfo(SceneData->IndirectionTexture.UAV, ERHIAccess::Unknown, ERHIAccess::UAVCompute));

				FComputeShaderUtils::Dispatch(RHICmdList, ComputeShader, Parameters, FIntVector(FMath::DivideAndRoundUp(SubLevelBrickPositions.Num(), 64), 1, 1));

				RHICmdList.Transition(FRHITransitionInfo(SceneData->IndirectionTexture.UAV, ERHIAccess::UAVCompute, ERHIAccess::SRVMask));

				ReleaseRHIForSubLevelResources();
			}
		}
		else
		{
			// CPU Path
			if (SceneData->IndirectionTexture.Data.Num() > 0)
			{
				for (int32 Index = 0; Index < SceneData->CPUSubLevelBrickDataList.Num(); ++Index)
				{
					if (SceneData->CPUSubLevelBrickDataList[Index] == this)
					{
						SceneData->CPUSubLevelBrickDataList[Index] = nullptr;
				
						for (int32 BrickIndex = 0; BrickIndex < SubLevelBrickPositions.Num(); BrickIndex++)
						{
							const FColor OriginalValue = IndirectionTextureOriginalValues[BrickIndex];

							const FIntVector IndirectionDestDataCoordinate = SubLevelBrickPositions[BrickIndex];
							const int32 IndirectionDestDataIndex =
								((IndirectionDestDataCoordinate.Z * SceneData->IndirectionTextureDimensions.Y) + IndirectionDestDataCoordinate.Y) *
								SceneData->IndirectionTextureDimensions.X + IndirectionDestDataCoordinate.X;

							{
								const int32 IndirectionTextureDataStride = GPixelFormats[SceneData->IndirectionTexture.Format].BlockBytes;
								uint8* IndirectionVoxelPtr = (uint8*)&SceneData->IndirectionTexture.Data[IndirectionDestDataIndex * IndirectionTextureDataStride];
								*(IndirectionVoxelPtr + 0) = OriginalValue.R;
								*(IndirectionVoxelPtr + 1) = OriginalValue.G;
								*(IndirectionVoxelPtr + 2) = OriginalValue.B;
								*(IndirectionVoxelPtr + 3) = OriginalValue.A;
							}

							{
								SceneData->CPUSubLevelIndirectionTable[IndirectionDestDataIndex] = 0;
							}
						}
					
						// we don't expect duplicates in CPUSubLevelBrickDataList
						break;
					}
				}
			}
		}
	}
}

SIZE_T FPrecomputedVolumetricLightmapData::GetAllocatedBytes() const
{
	return
		IndirectionTexture.DataSize + 
		BrickData.GetAllocatedBytes() + 
		SubLevelBrickPositions.Num() * SubLevelBrickPositions.GetTypeSize() +
		IndirectionTextureOriginalValues.Num() * IndirectionTextureOriginalValues.GetTypeSize();
}


FPrecomputedVolumetricLightmap::FPrecomputedVolumetricLightmap() :
	Data(NULL),
	bAddedToScene(false),
	WorldOriginOffset(ForceInitToZero)
{}

FPrecomputedVolumetricLightmap::~FPrecomputedVolumetricLightmap()
{
}

void FPrecomputedVolumetricLightmap::AddToScene(FSceneInterface* Scene, UMapBuildDataRegistry* Registry, FGuid LevelBuildDataId, bool bIsPersistentLevel)
{
	if (!IsStaticLightingAllowed())
	{
		return;
	}

	FPrecomputedVolumetricLightmapData* NewData = NULL;

	if (Registry)
	{
		NewData = Registry->GetLevelPrecomputedVolumetricLightmapBuildData(LevelBuildDataId);
	}

	if (NewData && Scene)
	{		
		AddToScene(Scene, Registry, NewData, bIsPersistentLevel);
	}	
}

void FPrecomputedVolumetricLightmap::AddToScene(FSceneInterface* Scene, UMapBuildDataRegistry* Registry, FPrecomputedVolumetricLightmapData* NewData, bool bIsPersistentLevel)
{
	if (!IsStaticLightingAllowed())
	{
		return;
	}

	// FIXME: temp fix for ordering issue between WorldContext.World()->InitWorld(); and GShaderCompilingManager->ProcessAsyncResults(false, true); in UnrealEngine.cpp
	if (GShaderCompilingManager)
	{
		GShaderCompilingManager->ProcessAsyncResults(true, true);
	}

	check(!bAddedToScene);

	if (NewData && Scene)
	{
		bAddedToScene = true;
		SourceRegistry = Registry;

		FPrecomputedVolumetricLightmap* Volume = this;

		ENQUEUE_RENDER_COMMAND(SetVolumeDataCommand)
			([Volume, NewData, Scene] (FRHICommandListBase& RHICmdList)
			{
				Volume->SetData(RHICmdList, NewData, Scene);
			});
		Scene->AddPrecomputedVolumetricLightmap(this, bIsPersistentLevel);
	}
}

void FPrecomputedVolumetricLightmap::RemoveFromScene(FSceneInterface* Scene)
{
	if (!IsStaticLightingAllowed())
	{
		return;
	}

	if (bAddedToScene)
	{
		bAddedToScene = false;

		// Certain paths in the editor (namely, ReloadPackages and ForceDelete) will GC the registry before the UWorld destruction (which destructs FScene)
		ensureMsgf(
			SourceRegistry.IsValid() // either SourceRegistry is valid
			|| Scene == nullptr // or there is no scene
			|| !Scene->GetWorld() || !IsValidChecked(Scene->GetWorld()) || Scene->GetWorld()->IsUnreachable() // or the world we're in is going away (usually during shutdown)
			, TEXT("UMapBuildDataRegistry is garbage collected before an FPrecomputedVolumetricLightmap is removed from the scene. Is there a missing ReleaseRenderingResources() call?"));

		// While that can be explained as missing ReleaseRenderingResources() calls, this fail-safe guard is added here
		if (SourceRegistry.IsValid())
		{
			SourceRegistry = nullptr;

			if (Scene)
			{
				Scene->RemovePrecomputedVolumetricLightmap(this);
			}
		}
	}

	WorldOriginOffset = FVector::ZeroVector;
}

void FPrecomputedVolumetricLightmap::SetData(FRHICommandListBase& RHICmdList, FPrecomputedVolumetricLightmapData* NewData, FSceneInterface* Scene)
{
	Data = NewData;

	if (Data)
	{
		Data->SetFeatureLevel(Scene->GetFeatureLevel());
		Data->IndirectionTexture.bNeedsCPUAccess = GIsEditor;
		Data->BrickData.SetNeedsCPUAccess(GIsEditor);

		if (Scene->GetFeatureLevel() >= ERHIFeatureLevel::SM5)
		{
			Data->InitResource(RHICmdList);
		}
	}
}

void FPrecomputedVolumetricLightmap::ApplyWorldOffset(const FVector& InOffset)
{
	WorldOriginOffset += InOffset;
}

FVector ComputeIndirectionCoordinate(FVector LookupPosition, const FBox& VolumeBounds, FIntVector IndirectionTextureDimensions)
{
	const FVector InvVolumeSize = FVector(1.0f) / VolumeBounds.GetSize();
	const FVector VolumeWorldToUVScale = InvVolumeSize;
	const FVector VolumeWorldToUVAdd = -VolumeBounds.Min * InvVolumeSize;

	FVector IndirectionDataSourceCoordinate = (LookupPosition * VolumeWorldToUVScale + VolumeWorldToUVAdd) * FVector(IndirectionTextureDimensions);
	IndirectionDataSourceCoordinate.X = FMath::Clamp<float>(IndirectionDataSourceCoordinate.X, 0.0f, IndirectionTextureDimensions.X - .01f);
	IndirectionDataSourceCoordinate.Y = FMath::Clamp<float>(IndirectionDataSourceCoordinate.Y, 0.0f, IndirectionTextureDimensions.Y - .01f);
	IndirectionDataSourceCoordinate.Z = FMath::Clamp<float>(IndirectionDataSourceCoordinate.Z, 0.0f, IndirectionTextureDimensions.Z - .01f);

	return IndirectionDataSourceCoordinate;
}

void SampleIndirectionTexture(
	FVector IndirectionDataSourceCoordinate,
	FIntVector IndirectionTextureDimensions,
	const uint8* IndirectionTextureData,
	FIntVector& OutIndirectionBrickOffset,
	int32& OutIndirectionBrickSize)
{
	FIntVector IndirectionDataCoordinateInt(IndirectionDataSourceCoordinate);
	
	IndirectionDataCoordinateInt.X = FMath::Clamp<int32>(IndirectionDataCoordinateInt.X, 0, IndirectionTextureDimensions.X - 1);
	IndirectionDataCoordinateInt.Y = FMath::Clamp<int32>(IndirectionDataCoordinateInt.Y, 0, IndirectionTextureDimensions.Y - 1);
	IndirectionDataCoordinateInt.Z = FMath::Clamp<int32>(IndirectionDataCoordinateInt.Z, 0, IndirectionTextureDimensions.Z - 1);

	const int32 IndirectionDataIndex = ((IndirectionDataCoordinateInt.Z * IndirectionTextureDimensions.Y) + IndirectionDataCoordinateInt.Y) * IndirectionTextureDimensions.X + IndirectionDataCoordinateInt.X;
	const uint8* IndirectionVoxelPtr = (const uint8*)&IndirectionTextureData[IndirectionDataIndex * sizeof(uint8) * 4];
	OutIndirectionBrickOffset = FIntVector(*(IndirectionVoxelPtr + 0), *(IndirectionVoxelPtr + 1), *(IndirectionVoxelPtr + 2));
	OutIndirectionBrickSize = *(IndirectionVoxelPtr + 3);
}

void SampleIndirectionTextureWithSubLevel(
	FVector IndirectionDataSourceCoordinate,
	FIntVector IndirectionTextureDimensions,
	const uint8* IndirectionTextureData,
	const TArray<uint8>& CPUSubLevelIndirectionTable,
	FIntVector& OutIndirectionBrickOffset,
	int32& OutIndirectionBrickSize,
	int32& OutSubLevelIndex)
{
	SampleIndirectionTexture(IndirectionDataSourceCoordinate, IndirectionTextureDimensions, IndirectionTextureData, OutIndirectionBrickOffset, OutIndirectionBrickSize);

	FIntVector IndirectionDataCoordinateInt(IndirectionDataSourceCoordinate);

	IndirectionDataCoordinateInt.X = FMath::Clamp<int32>(IndirectionDataCoordinateInt.X, 0, IndirectionTextureDimensions.X - 1);
	IndirectionDataCoordinateInt.Y = FMath::Clamp<int32>(IndirectionDataCoordinateInt.Y, 0, IndirectionTextureDimensions.Y - 1);
	IndirectionDataCoordinateInt.Z = FMath::Clamp<int32>(IndirectionDataCoordinateInt.Z, 0, IndirectionTextureDimensions.Z - 1);

	const int32 IndirectionDataIndex = ((IndirectionDataCoordinateInt.Z * IndirectionTextureDimensions.Y) + IndirectionDataCoordinateInt.Y) * IndirectionTextureDimensions.X + IndirectionDataCoordinateInt.X;

	OutSubLevelIndex = CPUSubLevelIndirectionTable[IndirectionDataIndex];
}

FVector ComputeBrickTextureCoordinate(
	FVector IndirectionDataSourceCoordinate,
	FIntVector IndirectionBrickOffset, 
	int32 IndirectionBrickSize,
	int32 BrickSize)
{
	FVector IndirectionDataSourceCoordinateInBricks = IndirectionDataSourceCoordinate / IndirectionBrickSize;
	FVector FractionalIndirectionDataCoordinate(FMath::Frac(IndirectionDataSourceCoordinateInBricks.X), FMath::Frac(IndirectionDataSourceCoordinateInBricks.Y), FMath::Frac(IndirectionDataSourceCoordinateInBricks.Z));
	int32 PaddedBrickSize = BrickSize + 1;
	FVector BrickTextureCoordinate = FVector(IndirectionBrickOffset * PaddedBrickSize) + FractionalIndirectionDataCoordinate * BrickSize;
	return BrickTextureCoordinate;
}

bool FRemoveSubLevelBricksCS::ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
{
	return IsFeatureLevelSupported(Parameters.Platform, ERHIFeatureLevel::SM5) && IsStaticLightingAllowed();
}

bool FCopyResidentBricksCS::ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
{
	return IsFeatureLevelSupported(Parameters.Platform, ERHIFeatureLevel::SM5) && IsStaticLightingAllowed();
}

bool FCopyResidentBrickSHCoefficientsCS::ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
{
	return IsFeatureLevelSupported(Parameters.Platform, ERHIFeatureLevel::SM5) && IsStaticLightingAllowed();
}

bool FPatchIndirectionTextureCS::ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
{
	return IsFeatureLevelSupported(Parameters.Platform, ERHIFeatureLevel::SM5) && IsStaticLightingAllowed();
}

bool FMoveWholeIndirectionTextureCS::ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
{
	return IsFeatureLevelSupported(Parameters.Platform, ERHIFeatureLevel::SM5) && IsStaticLightingAllowed();
}

IMPLEMENT_GLOBAL_SHADER(FRemoveSubLevelBricksCS, "/Engine/Private/VolumetricLightmapStreaming.usf", "RemoveSubLevelBricksCS", SF_Compute);
IMPLEMENT_GLOBAL_SHADER(FCopyResidentBricksCS, "/Engine/Private/VolumetricLightmapStreaming.usf", "CopyResidentBricksCS", SF_Compute);
IMPLEMENT_GLOBAL_SHADER(FCopyResidentBrickSHCoefficientsCS, "/Engine/Private/VolumetricLightmapStreaming.usf", "CopyResidentBrickSHCoefficientsCS", SF_Compute);
IMPLEMENT_GLOBAL_SHADER(FPatchIndirectionTextureCS, "/Engine/Private/VolumetricLightmapStreaming.usf", "PatchIndirectionTextureCS", SF_Compute);
IMPLEMENT_GLOBAL_SHADER(FMoveWholeIndirectionTextureCS, "/Engine/Private/VolumetricLightmapStreaming.usf", "MoveWholeIndirectionTextureCS", SF_Compute);

FVolumetricLightmapBrickAtlas::FVolumetricLightmapBrickAtlas()
	: bInitialized(false)
	, PaddedBrickSize(5)
{
}

template<class VolumetricLightmapBrickDataType>
void CopyDataIntoAtlas(FRHICommandList& RHICmdList, ERHIFeatureLevel::Type FeatureLevel, int32 SrcOffset, int32 DestOffset, int32 NumBricks, const VolumetricLightmapBrickDataType& SrcData, FVolumetricLightmapBrickTextureSet DestTextureSet)
{
	TArray<FRHITransitionInfo, SceneRenderingAllocator> Infos;
	Infos.Reserve(3 + UE_ARRAY_COUNT(SrcData.SHCoefficients));
	Infos.Emplace(DestTextureSet.AmbientVector.UAV,             ERHIAccess::Unknown, ERHIAccess::UAVCompute);
	Infos.Emplace(DestTextureSet.SkyBentNormal.UAV,             ERHIAccess::Unknown, ERHIAccess::UAVCompute);
	Infos.Emplace(DestTextureSet.DirectionalLightShadowing.UAV, ERHIAccess::Unknown, ERHIAccess::UAVCompute);

	for (int32 i = 0; i < UE_ARRAY_COUNT(SrcData.SHCoefficients); i++)
	{
		Infos.Emplace(DestTextureSet.SHCoefficients[i].UAV, ERHIAccess::Unknown, ERHIAccess::UAVCompute);
	}
	RHICmdList.Transition(Infos);

	FGlobalShaderMap* GlobalShaderMap = GetGlobalShaderMap(FeatureLevel);

	for (int32 BatchOffset = 0; BatchOffset < NumBricks; BatchOffset += GRHIMaxDispatchThreadGroupsPerDimension.X)
	{
		{
			FCopyResidentBricksCS::FPermutationDomain PermutationVector;
			PermutationVector.Set<FCopyResidentBricksCS::FHasSkyBentNormal>(SrcData.SkyBentNormal.Texture.IsValid());

			TShaderMapRef<FCopyResidentBricksCS> ComputeShader(GlobalShaderMap, PermutationVector);

			FCopyResidentBricksCS::FParameters Parameters;

			Parameters.StartPosInOldVolume = BatchOffset + SrcOffset;
			Parameters.StartPosInNewVolume = BatchOffset + DestOffset;

			Parameters.AmbientVector = SrcData.AmbientVector.Texture;
			Parameters.SkyBentNormal = SrcData.SkyBentNormal.Texture;
			Parameters.DirectionalLightShadowing = SrcData.DirectionalLightShadowing.Texture;

			Parameters.OutAmbientVector = DestTextureSet.AmbientVector.UAV;
			Parameters.OutSkyBentNormal = DestTextureSet.SkyBentNormal.UAV;
			Parameters.OutDirectionalLightShadowing = DestTextureSet.DirectionalLightShadowing.UAV;

			FComputeShaderUtils::Dispatch(RHICmdList, ComputeShader, Parameters, FIntVector(FMath::Min(NumBricks, GRHIMaxDispatchThreadGroupsPerDimension.X), 1, 1));
		}

		for (int32 i = 0; i < UE_ARRAY_COUNT(SrcData.SHCoefficients); i++)
		{
			TShaderMapRef<FCopyResidentBrickSHCoefficientsCS> ComputeShader(GlobalShaderMap);

			FCopyResidentBrickSHCoefficientsCS::FParameters Parameters;

			Parameters.StartPosInOldVolume = BatchOffset + SrcOffset;
			Parameters.StartPosInNewVolume = BatchOffset + DestOffset;

			Parameters.SHCoefficients = SrcData.SHCoefficients[i].Texture;
			Parameters.OutSHCoefficients = DestTextureSet.SHCoefficients[i].UAV;

			FComputeShaderUtils::Dispatch(RHICmdList, ComputeShader, Parameters, FIntVector(FMath::Min(NumBricks, GRHIMaxDispatchThreadGroupsPerDimension.X), 1, 1));
		}
	}

	// Make all the resources readable again
	for (FRHITransitionInfo& Info : Infos)
	{
		Info.AccessBefore = Info.AccessAfter;
		Info.AccessAfter = ERHIAccess::SRVMask;
	}
	RHICmdList.Transition(Infos);
}

void FVolumetricLightmapBrickAtlas::Insert(FRHICommandListImmediate& RHICmdList, int32 Index, FPrecomputedVolumetricLightmapData* Data)
{
	check(!Allocations.FindByPredicate([Data](const Allocation& Other) { return Other.Data == Data; }));

	bool bReadAfterCreate = false;

	if (!bInitialized)
	{
		SetFeatureLevel(GMaxRHIFeatureLevel);
		check(Data->BrickSize > 0);
		PaddedBrickSize = Data->BrickSize + 1;
		TextureSet.Initialize(Data->BrickDataDimensions, Data->BrickData);
		bInitialized = true;
		bReadAfterCreate = true;
	}
	else
	{
		check(TextureSet.AmbientVector.Format == Data->BrickData.AmbientVector.Format);
		for (int32 i = 0; i < UE_ARRAY_COUNT(TextureSet.SHCoefficients); i++)
		{
			check(TextureSet.SHCoefficients[i].Format == Data->BrickData.SHCoefficients[i].Format);
		}
		check(TextureSet.SkyBentNormal.Format == Data->BrickData.SkyBentNormal.Format);
		check(TextureSet.DirectionalLightShadowing.Format == Data->BrickData.DirectionalLightShadowing.Format);

		// If the incoming BrickData has sky bent normal, also create one in atlas
		// TODO: release SkyBentNormal if no brick data in the atlas uses it
		if (!TextureSet.SkyBentNormal.Texture.IsValid() && Data->BrickData.SkyBentNormal.Texture.IsValid())
		{
			TextureSet.SkyBentNormal.CreateTargetTexture(RHICmdList, TextureSet.BrickDataDimensions);
			TextureSet.SkyBentNormal.CreateUAV(RHICmdList);
			bReadAfterCreate = true;
		}
	}

	if (bReadAfterCreate)
	{
		FRHITransitionInfo Transitions[UE_ARRAY_COUNT(TextureSet.SHCoefficients) + 3] =
		{
			FRHITransitionInfo(TextureSet.AmbientVector.Texture, ERHIAccess::Unknown, ERHIAccess::SRVMask),
			FRHITransitionInfo(TextureSet.SkyBentNormal.Texture, ERHIAccess::Unknown, ERHIAccess::SRVMask),
			FRHITransitionInfo(TextureSet.DirectionalLightShadowing.Texture, ERHIAccess::Unknown, ERHIAccess::SRVMask)
		};
		for (int32 i = 0; i < UE_ARRAY_COUNT(TextureSet.SHCoefficients); i++)
		{
			Transitions[i + 3] = FRHITransitionInfo(TextureSet.SHCoefficients[i].Texture, ERHIAccess::Unknown, ERHIAccess::SRVMask);
		}
		RHICmdList.Transition(MakeArrayView(Transitions, UE_ARRAY_COUNT(Transitions)));
	}

	int32 NumTotalBricks = 0;

	for (const auto& Allocation : Allocations)
	{
		NumTotalBricks += Allocation.Size;
	}

	NumTotalBricks += Data->BrickDataDimensions.X * Data->BrickDataDimensions.Y * Data->BrickDataDimensions.Z / (PaddedBrickSize * PaddedBrickSize * PaddedBrickSize);

	TArray<Allocation> NewAllocations;
	FVolumetricLightmapBrickTextureSet NewTextureSet;

	{
		const int32 MaxBricksInLayoutOneDim = 1 << 8;
		int32 BrickTextureLinearAllocator = NumTotalBricks;
		FIntVector BrickLayoutDimensions;
		BrickLayoutDimensions.X = FMath::Min(BrickTextureLinearAllocator, MaxBricksInLayoutOneDim);
		BrickTextureLinearAllocator = FMath::DivideAndRoundUp(BrickTextureLinearAllocator, BrickLayoutDimensions.X);
		BrickLayoutDimensions.Y = FMath::Min(BrickTextureLinearAllocator, MaxBricksInLayoutOneDim);
		BrickTextureLinearAllocator = FMath::DivideAndRoundUp(BrickTextureLinearAllocator, BrickLayoutDimensions.Y);
		BrickLayoutDimensions.Z = FMath::Min(BrickTextureLinearAllocator, MaxBricksInLayoutOneDim);
		FIntVector BrickDataDimensions = BrickLayoutDimensions * PaddedBrickSize;

		NewTextureSet.Initialize(BrickDataDimensions, TextureSet);
	}

	// Dry run to handle persistent level data movement properly
	{
		int32 BrickStartAllocation = 0;

		// Copy old allocations
		for (int32 AllocationIndex = 0; AllocationIndex < Index && AllocationIndex < Allocations.Num(); AllocationIndex++)
		{
			BrickStartAllocation += Allocations[AllocationIndex].Size;
		}

		// Insert new allocation
		{
			int32 NumBricks = Data->BrickDataDimensions.X * Data->BrickDataDimensions.Y * Data->BrickDataDimensions.Z / (PaddedBrickSize * PaddedBrickSize * PaddedBrickSize);

			BrickStartAllocation += NumBricks;
		}

		// Copy the rest of allocations
		for (int32 AllocationIndex = Index; AllocationIndex < Allocations.Num(); AllocationIndex++)
		{
			if (Allocations[AllocationIndex].Data->IndirectionTextureDimensions.GetMax() > 0)
			{
				Allocations[AllocationIndex].Data->HandleDataMovementInAtlas(RHICmdList, Allocations[AllocationIndex].StartOffset, BrickStartAllocation);
			}
			BrickStartAllocation += Allocations[AllocationIndex].Size;
		}
	}

	{
		int32 BrickStartAllocation = 0;

		// Copy old allocations
		for (int32 AllocationIndex = 0; AllocationIndex < Index && AllocationIndex < Allocations.Num(); AllocationIndex++)
		{
			CopyDataIntoAtlas(RHICmdList, GetFeatureLevel(), Allocations[AllocationIndex].StartOffset, BrickStartAllocation, Allocations[AllocationIndex].Size, TextureSet, NewTextureSet);

			NewAllocations.Add(Allocation{ Allocations[AllocationIndex].Data, Allocations[AllocationIndex].Size, BrickStartAllocation });
			BrickStartAllocation += Allocations[AllocationIndex].Size;
		}

		// Insert new allocation
		{
			int32 NumBricks = Data->BrickDataDimensions.X * Data->BrickDataDimensions.Y * Data->BrickDataDimensions.Z / (PaddedBrickSize * PaddedBrickSize * PaddedBrickSize);

			CopyDataIntoAtlas(RHICmdList, GetFeatureLevel(), 0, BrickStartAllocation, NumBricks, Data->BrickData, NewTextureSet);

			NewAllocations.Add(Allocation{ Data, NumBricks, BrickStartAllocation });
			Data->BrickDataBaseOffsetInAtlas = BrickStartAllocation;
			BrickStartAllocation += NumBricks;
		}

		// Copy the rest of allocations
		for (int32 AllocationIndex = Index; AllocationIndex < Allocations.Num(); AllocationIndex++)
		{
			CopyDataIntoAtlas(RHICmdList, GetFeatureLevel(), Allocations[AllocationIndex].StartOffset, BrickStartAllocation, Allocations[AllocationIndex].Size, TextureSet, NewTextureSet);

			NewAllocations.Add(Allocation{ Allocations[AllocationIndex].Data, Allocations[AllocationIndex].Size, BrickStartAllocation });
			// Handle the sub level data movements
			if (Allocations[AllocationIndex].Data->IndirectionTextureDimensions.GetMax() == 0)
			{
				Allocations[AllocationIndex].Data->HandleDataMovementInAtlas(RHICmdList, Allocations[AllocationIndex].StartOffset, BrickStartAllocation);
			}
			BrickStartAllocation += Allocations[AllocationIndex].Size;
		}
	}

	// Replace with new allcations
	Allocations = NewAllocations;
	TextureSet = NewTextureSet; // <-- Old texture references are released here
}

void FVolumetricLightmapBrickAtlas::Remove(FRHICommandListImmediate& RHICmdList, FPrecomputedVolumetricLightmapData* Data)
{
	Allocation* AllocationEntry = Allocations.FindByPredicate([Data](const Allocation& Other) { return Other.Data == Data; });
	if (!AllocationEntry)
	{
		return;
	}

	int32 Index = (int32)(AllocationEntry - Allocations.GetData());

	int32 NumTotalBricks = 0;

	for (const auto& Allocation : Allocations)
	{
		if (Allocation.Data != Data)
		{
			NumTotalBricks += Allocation.Size;
		}
	}

	TArray<Allocation> NewAllocations;
	FVolumetricLightmapBrickTextureSet NewTextureSet;

	if (NumTotalBricks > 0)
	{
		{
			const int32 MaxBricksInLayoutOneDim = 1 << 8;
			int32 BrickTextureLinearAllocator = NumTotalBricks;
			FIntVector BrickLayoutDimensions;
			BrickLayoutDimensions.X = FMath::Min(BrickTextureLinearAllocator, MaxBricksInLayoutOneDim);
			BrickTextureLinearAllocator = FMath::DivideAndRoundUp(BrickTextureLinearAllocator, BrickLayoutDimensions.X);
			BrickLayoutDimensions.Y = FMath::Min(BrickTextureLinearAllocator, MaxBricksInLayoutOneDim);
			BrickTextureLinearAllocator = FMath::DivideAndRoundUp(BrickTextureLinearAllocator, BrickLayoutDimensions.Y);
			BrickLayoutDimensions.Z = FMath::Min(BrickTextureLinearAllocator, MaxBricksInLayoutOneDim);
			FIntVector BrickDataDimensions = BrickLayoutDimensions * PaddedBrickSize;

			NewTextureSet.Initialize(BrickDataDimensions, TextureSet);
		}

		{
			int32 BrickStartAllocation = 0;

			// Copy old allocations
			for (int32 AllocationIndex = 0; AllocationIndex < Index && AllocationIndex < Allocations.Num(); AllocationIndex++)
			{
				CopyDataIntoAtlas(RHICmdList, GetFeatureLevel(), Allocations[AllocationIndex].StartOffset, BrickStartAllocation, Allocations[AllocationIndex].Size, TextureSet, NewTextureSet);

				NewAllocations.Add(Allocation{ Allocations[AllocationIndex].Data, Allocations[AllocationIndex].Size, BrickStartAllocation });
				BrickStartAllocation += Allocations[AllocationIndex].Size;
			}

			// Skip the allocation being deleted

			// Copy the rest of allocations
			for (int32 AllocationIndex = Index + 1; AllocationIndex < Allocations.Num(); AllocationIndex++)
			{
				CopyDataIntoAtlas(RHICmdList, GetFeatureLevel(), Allocations[AllocationIndex].StartOffset, BrickStartAllocation, Allocations[AllocationIndex].Size, TextureSet, NewTextureSet);

				NewAllocations.Add(Allocation{ Allocations[AllocationIndex].Data, Allocations[AllocationIndex].Size, BrickStartAllocation });
				Allocations[AllocationIndex].Data->HandleDataMovementInAtlas(RHICmdList, Allocations[AllocationIndex].StartOffset, BrickStartAllocation);
				BrickStartAllocation += Allocations[AllocationIndex].Size;
			}
		}
	}
	else
	{
		bInitialized = false;
	}

	// Replace with new allcations
	Allocations = NewAllocations;
	TextureSet = NewTextureSet; // <-- Old texture references are released here
}

void FVolumetricLightmapBrickAtlas::ReleaseRHI()
{
	TArray<Allocation> AllocationList = Allocations;
	for (Allocation& Alloc : AllocationList)
	{
		Remove(FRHICommandListImmediate::Get(), Alloc.Data);
	}

	TextureSet.Release();
}
