// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	PrecomputedVolumetricLightmap.cpp
=============================================================================*/

#include "PrecomputedVolumetricLightmapStreaming.h"
#include "PrecomputedVolumetricLightmap.h"
#include "Engine/World.h"
#include "Engine/Level.h"
#include "ContentStreaming.h"
#include "WorldPartition/StaticLightingData/VolumetricLightmapGrid.h"
#include "Serialization/VersionedArchive.h"
#include "Serialization/MemoryReader.h"
#include "GameFramework/WorldSettings.h"
#include "ProfilingDebugging/IoStoreTrace.h"

using FVersionedMemoryReaderView = TVersionedReader<FMemoryReaderView>;

DEFINE_LOG_CATEGORY_STATIC(LogVolumetricLightmapStreaming, Warning, All);

enum ETimedExecutionControl
{
	Continue, 
	Restart, 
	Stop,
};

template<class ItemCollectionType, class ExecuteFn>
bool TimedExecution(ItemCollectionType& Items, float TimeLimit, const ExecuteFn& Execute) 		
{
	double EndTime = FPlatformTime::Seconds() + TimeLimit;	
	if (TimeLimit == 0)
	{
		EndTime = FLT_MAX;
	}
	
	bool bContinue = true; 
	
	while (bContinue)	
	{
		bContinue = false; 

		for (auto& It : Items)
		{
			float ThisTimeLimit = static_cast<float>(EndTime - FPlatformTime::Seconds());

			if (ThisTimeLimit < .001f) // one ms is the granularity of the platform event system
			{
				return false;
			}

			ETimedExecutionControl Control = Execute(ThisTimeLimit, It);

			if (Control == ETimedExecutionControl::Restart)
			{
				bContinue = true;
				break;
			}
			else if (Control == ETimedExecutionControl::Stop)
			{
				break;
			}			
		}
	}
		
	return true;
}

class FVolumetricLightmapGridStreamingManager : public IStreamingManager
{
public:

	friend class FVolumetricLightmapGridManager;
	
	FVolumetricLightmapGridManager* Owner;

	FVolumetricLightmapGridStreamingManager(FVolumetricLightmapGridManager* InOwner)
		: Owner(InOwner)
	{
		IStreamingManager::Get().AddStreamingManager(this);
	}

	~FVolumetricLightmapGridStreamingManager()
	{
		IStreamingManager::Get().RemoveStreamingManager(this);
	}

	virtual void Tick(float DeltaTime, bool bProcessEverything = false) override
	{
		
	}

	virtual void UpdateResourceStreaming(float DeltaTime, bool bProcessEverything = false) override
	{
		 int32 NbViews = IStreamingManager::Get().GetNumViews();
		 UWorld* ViewWorld = Owner->World;
		 const FStreamingViewInfo* ViewInfoForWorld = nullptr;
		 const FStreamingViewInfo* ViewInfoNoWorld = nullptr;

		 // @todo_ow: Improve multiple views handling, add better render/world setting to determine extent / budget
		 // Find the best ViewInfo for the OwnerWorld
		 for (int i = 0; i < NbViews; i++)
		 {
			 const FStreamingViewInfo& ViewInfo = IStreamingManager::Get().GetViewInformation(i);

			 if (ViewInfo.World == ViewWorld && !ViewInfoForWorld)
			 {
				 ViewInfoForWorld = &ViewInfo;
				 break;
			 }
			 else if (!ViewInfo.World.IsValid() && !ViewInfoNoWorld)
			 {
				 ViewInfoNoWorld = &ViewInfo;
			 }
		 }

		 if (ViewInfoForWorld || ViewInfoNoWorld)
		 {
			 const FStreamingViewInfo& ViewInfo = ViewInfoForWorld ? *ViewInfoForWorld : *ViewInfoNoWorld;
			 FBox::FReal StreamDistance = ViewWorld->GetWorldSettings()->VolumetricLightmapLoadingRange;
			 FBox Bounds = FBox(ViewInfo.ViewOrigin - FVector(StreamDistance, StreamDistance, StreamDistance), ViewInfo.ViewOrigin + FVector(StreamDistance, StreamDistance, StreamDistance));
			 Owner->UpdateBounds(Bounds);
		 }
	}

	virtual int32 BlockTillAllRequestsFinished(float TimeLimit = 0.0f, bool bLogResults = false) override
	{
		return Owner->WaitForPendingRequest(TimeLimit);
	}

	virtual void CancelForcedResources() override {}
	virtual void NotifyLevelChange() override {}
	virtual void SetDisregardWorldResourcesForFrames(int32 NumFrames) override {}
	virtual void AddLevel(class ULevel* Level) override {}
	virtual void RemoveLevel(class ULevel* Level) override {}

	virtual void NotifyLevelOffset(class ULevel* Level, const FVector& Offset) override { check(false); /* Unsupported */ } 
	virtual int32 GetNumWantingResources() const override
	{ 
		return Owner->GetNumPendingRequests();
	}
};


FVolumetricLightmapGridManager::FVolumetricLightmapGridManager(UWorld* InWorld, FVolumetricLightMapGridDesc* InGrid)
	: World(InWorld)
	, Registry(InWorld->PersistentLevel->MapBuildData)
	, Grid(InGrid)
{
	StreamingManager = MakeUnique<FVolumetricLightmapGridStreamingManager>(this);
}

FVolumetricLightmapGridManager::~FVolumetricLightmapGridManager()
{
	check(LoadedCells.IsEmpty());
}

int32 FVolumetricLightmapGridManager::GetNumPendingRequests()
{
	FScopeLock PendingRequestsLock(&PendingRequestsCriticalSection);
	return PendingCellRequests.Num();
}

int32 FVolumetricLightmapGridManager::WaitForPendingRequest(float TimeLimit)
{
	bool bTimedOut = TimedExecution(PendingCellRequests, TimeLimit, [this](float TimeLimit, FCellRequest& Request) -> ETimedExecutionControl
	{		
		if (Request.IORequest && Request.IORequest->WaitCompletion(TimeLimit))
		{	
			ProcessRequests();
			return ETimedExecutionControl::Restart; // Restart iteration once requests have been processded since PendingCellRequests is modified by ProcessRequests()
		}

		return ETimedExecutionControl::Continue;
	});

	return PendingCellRequests.Num();
}


void FVolumetricLightmapGridManager::ReleaseCellData(FLoadedCellData* LoadedCell, FSceneInterface* InScene)
{
	if (LoadedCell->Data)
	{
		FVolumetricLightMapGridCell* Cell = LoadedCell->Cell;
		UE_LOGF(LogVolumetricLightmapStreaming, Log, "Releasing cell data for streaming cell %i (%p, %ls)", Cell->CellID, Cell, *World->GetFullName());

		FPrecomputedVolumetricLightmapData* Data = LoadedCell->Data;
		LoadedCell->Data = nullptr;

		ENQUEUE_RENDER_COMMAND(DeleteVolumetricLightDataCommand)
			([Data] (FRHICommandListBase&)
		{
			Data->ReleaseResource();
			delete(Data);
		});
	}
}

void FVolumetricLightmapGridManager::RemoveFromScene(FSceneInterface* InScene)
{
	// In the unlikely event we have pending requests, wait for pending requests to finish so that we can release the data properly
	WaitForPendingRequest(0.0f);

	for (auto& It : LoadedCells)
	{
		FLoadedCellData& LoadedCell = It.Value;
		FVolumetricLightMapGridCell* GridCell = It.Key;

		if (LoadedCell.Lightmap)
		{
			LoadedCell.Lightmap->RemoveFromScene(InScene);
		}

		ReleaseCellData(&LoadedCell, InScene);
	}

	LoadedCells.Reset();

	StreamingManager.Reset();	
}

void FVolumetricLightmapGridManager::RequestVolumetricLightMapCell(FCellRequest& InCellRequest)
{		
	FVolumetricLightMapGridCell* Cell = InCellRequest.Cell;
	check(Cell);

	// should not be already loaded
	check(!InCellRequest.Data);

	if (!Cell->BulkData.GetElementCount())
	{
		InCellRequest.Status = FCellRequest::Ready;
		return;
	}
		
	FBulkDataIORequestCallBack RequestCallback = [Cell, this](bool bWasCancelled, IBulkDataIORequest* IORequest)
	{
		if (!bWasCancelled)
		{
			void* Memory = IORequest->GetReadResults();
			if (Memory)
			{
				FMemoryView MemoryView(Memory, IORequest->GetSize());

				FVersionedMemoryReaderView FileDataAr(MemoryView, true);
				FPrecomputedVolumetricLightmapData* Data = nullptr;	

				FileDataAr << Data;

				{
					FScopeLock PendingRequestsLock(&PendingRequestsCriticalSection);

					FCellRequest* Request = PendingCellRequests.FindByPredicate([Cell](FCellRequest& Request) { return Request.Cell == Cell; });
					check(Request);
					
					UE_LOGF(LogVolumetricLightmapStreaming, Log, "IO Request Callback for streaming cell %i (%p, %ls)", Cell->CellID, Cell, *this->World->GetFullName());

					check(Data);
					check(!Request->Data); // redundant check added because this runs in the async thread
					Request->Data = Data;
				}
				
				FMemory::Free(Memory);
			}
		}
		else
		{
			check(!IORequest->GetReadResults());
		}
	};

	TRACE_IOSTORE_METADATA_SCOPE_TAG("PrecomputedVolumetricLightmap");

	if (!Cell->BulkData.IsBulkDataLoaded())
	{
		UE_LOGF(LogVolumetricLightmapStreaming, Log, "Request streaming for cell %i (%p, %ls)", Cell->CellID, Cell, *World->GetFullName());
		InCellRequest.IORequest = Cell->BulkData.CreateStreamingRequest(AIOP_Normal, &RequestCallback, nullptr);
		InCellRequest.Status = FCellRequest::Requested;
	}
#if WITH_EDITOR
	else
	{
		UE_LOGF(LogVolumetricLightmapStreaming, Log, "Loading streaming cell %i (%p, %ls) without streaming it", Cell->CellID, Cell, *World->GetFullName());
		// For unsaved data we can't stream it, we need to do an immediate load (Cell.BulkData.IsBulkDataLoaded() will always return true for unsaved data)
		Grid->LoadVolumetricLightMapCell(*Cell, InCellRequest.Data);
		InCellRequest.Status = FCellRequest::Ready;
	}
#else
	else
	{	
		// We should never end up in this state in non-editor builds
		check(false);
		InCellRequest.Status = FCellRequest::Ready;
	}
#endif
}

void FVolumetricLightmapGridManager::UpdateBounds(const FBox& InBounds)
{
	UE_LOGF(LogVolumetricLightmapStreaming, Verbose, "Updating bounds: %ls, for : %ls", *InBounds.ToString(), *World->GetFullName());

	FScopeLock PendingRequestsLock(&PendingRequestsCriticalSection);

	//@todo_ow: Make a pass on this to minimize heap allocations
	check(Grid);	

	// @todo_ow: Add check to see if bounds changed enough? there's a whole bunch of logic around this we'll see if we do it later
	TArray<FVolumetricLightMapGridCell*> IntersectingCells = Grid->GetIntersectingCells(InBounds, true);

	// Build list of cells to add & remove	
	TArray<FVolumetricLightMapGridCell*> CellsToRequest;
	
	TSet<FVolumetricLightMapGridCell*>  CellsToRemove;	
	LoadedCells.GetKeys(CellsToRemove);

	for (FVolumetricLightMapGridCell* Cell : IntersectingCells)
	{
		if (FLoadedCellData* LoadedCell = LoadedCells.Find(Cell))
		{		
			CellsToRemove.Remove(Cell);
		}
		else 
		{
			CellsToRequest.Add(Cell);
		}
	}

	// Make the necessary IO requests for new cells
	for (FVolumetricLightMapGridCell* Cell : CellsToRequest)
	{
		if (!PendingCellRequests.ContainsByPredicate([Cell](FCellRequest& Request) { return Request.Cell == Cell; }))
		{
			FCellRequest Request { .Cell = Cell };

			RequestVolumetricLightMapCell(Request);

			PendingCellRequests.Add(Request);
		}
	}

#if DO_CHECK
	// Since cells to remove are obtained from subtracting all IntersectingCells from the LoadedCells set, pending requested cells 
	// should never be in CellsToRemove
	for (FVolumetricLightMapGridCell* Cell : CellsToRemove)
	{	
		check(!PendingCellRequests.FindByPredicate([Cell](FCellRequest& InRequest) { return InRequest.Cell == Cell; }));
	}
#endif

	// Since cells to remove are obtained from subtracting all IntersectingCells from the LoadedCells set, pending requested cells are never
	// in CellsToRemove, so this bit is not useful for now. We could extract cells to remove from the pending rq
	// @todo_ow: Handle removal of requested cells, an optimization for rarer/edgier cases so we'll do this later, plus this ones needs actual synchronization with the async request
	/*for (FVolumetricLightMapGridCell* Cell : CellsToRemove)
	{	
		if (FCellRequest* Request = PendingCellRequests.FindByPredicate([Cell](FCellRequest& InRequest) { return InRequest.Cell == Cell; }))
		{
			check(Request->Status == FCellRequest::Requested);
			Request->Status = FCellRequest::Cancelled;			
			Request->IORequest->Cancel();
			delete Request->IORequest;
			Request->IORequest = nullptr;
		}
	}*/
	
	// Remove all unnecessary cells
	for (FVolumetricLightMapGridCell* Cell : CellsToRemove)
	{
		FLoadedCellData& LoadedCell = LoadedCells[Cell];
		if (LoadedCell.Lightmap)
		{
			LoadedCell.Lightmap->RemoveFromScene(World->Scene);
		}

		verify(LoadedCells.Remove(Cell) > 0);

		ReleaseCellData(&LoadedCell, World->Scene);
	}

	// Update currently tracked bounds
	Bounds = InBounds;

	ProcessRequests();
}

int32 FVolumetricLightmapGridManager::ProcessRequests()
{	
	FScopeLock PendingRequestsLock(&PendingRequestsCriticalSection);
	
	TArray<FCellRequest*> CellsToAdd;

	// Process pending IO requests and move to add list if ready
	//@todo_ow: Use an inline array and just mark the requests that need to be removed
	TArray<FCellRequest> UpdatedRequests;

	for (FCellRequest& Request : PendingCellRequests)
	{
		if (Request.Status == FCellRequest::Ready)
		{
			check(Request.Data);
			check(!Request.IORequest);
			CellsToAdd.Add(&Request);
		}
		else if (Request.Status == FCellRequest::Requested)
		{
			if (Request.IORequest->PollCompletion())
			{
				check(Request.Data);
				CellsToAdd.Add(&Request);
				
				delete Request.IORequest;
				Request.IORequest = nullptr;
				Request.Status = FCellRequest::Ready;
			}
			else
			{
				UpdatedRequests.Add(Request);
			}
		}
		else
		{
			checkf(false, TEXT("Unexpected request status\n"));
		}
	}

	// Add all necessary cells
	for (FCellRequest* FCellRequest : CellsToAdd)
	{
		FLoadedCellData LoadedCell { .Data = FCellRequest->Data, .Cell = FCellRequest->Cell };

		if (LoadedCell.Data)
		{
			LoadedCell.Lightmap = new FPrecomputedVolumetricLightmap();
			LoadedCell.Lightmap->AddToScene(World->Scene, Registry, LoadedCell.Data, false);
		}
		else
		{
			check(LoadedCell.Cell->BulkData.GetElementCount() == 0);
		}

		LoadedCells.Add(LoadedCell.Cell, LoadedCell);
	}

	// We can now overwrite the pending requests array since we're not using ptrs into it anymore
	PendingCellRequests = MoveTemp(UpdatedRequests);

	return PendingCellRequests.Num();
}