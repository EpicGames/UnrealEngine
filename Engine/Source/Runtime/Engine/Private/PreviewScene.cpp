// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	PreviewScene.cpp: Preview scene implementation.
=============================================================================*/

#include "PreviewScene.h"
#include "Engine/Engine.h"
#include "Engine/GameInstance.h"
#include "Misc/ConfigCacheIni.h"
#include "UObject/Package.h"
#include "SceneInterface.h"
#include "Components/StaticMeshComponent.h"
#include "AudioDevice.h"
#include "Components/DirectionalLightComponent.h"
#include "Components/SkyLightComponent.h"
#include "Components/LineBatchComponent.h"
#include "Components/ReflectionCaptureComponent.h"
#include "GameFramework/GameModeBase.h"

namespace PreviewScene::Private
{
	bool bForceGCOnPreviewSceneExit = true;
	FAutoConsoleVariableRef CVarForceGCOnPreviewSceneExit(TEXT("r.ForceGCOnPreviewSceneExit"), bForceGCOnPreviewSceneExit, TEXT(""), ECVF_Default);

}

FPreviewScene::FPreviewScene(FPreviewScene::ConstructionValues CVS)
	: PreviewWorld(nullptr)
	, bForceAllUsedMipsResident(CVS.bForceMipsResident)
{
	EObjectFlags NewObjectFlags = RF_NoFlags;
	if (CVS.bTransactional)
	{
		NewObjectFlags = RF_Transactional;
	}

	PreviewWorld = NewObject<UWorld>(GetTransientPackage(), NAME_None, NewObjectFlags);
	PreviewWorld->WorldType = CVS.bEditor ? EWorldType::EditorPreview : EWorldType::GamePreview;

	FWorldContext& WorldContext = GEngine->CreateNewWorldContext(PreviewWorld->WorldType);
	WorldContext.SetCurrentWorld(PreviewWorld);

	PreviewWorld->InitializeNewWorld(UWorld::InitializationValues()
										.AllowAudioPlayback(CVS.bAllowAudioPlayback)
										.CreatePhysicsScene(CVS.bCreatePhysicsScene)
										.RequiresHitProxies(CVS.bEditor) // Only Need hit proxies in an editor scene
										.CreateNavigation(false)
										.CreateAISystem(false)
										.ShouldSimulatePhysics(CVS.bShouldSimulatePhysics)
										.SetTransactional(CVS.bTransactional)
										.SetDefaultGameMode(CVS.DefaultGameMode)
										.ForceUseMovementComponentInNonGameWorld(CVS.bForceUseMovementComponentInNonGameWorld)
										.AllowLumenPrimitiveTrackingInPreviewWorld(CVS.bAllowLumenPrimitiveTrackingInPreviewWorld));

	FURL URL = FURL();
	//URL += TEXT("?SpectatorOnly=1");
	//URL = FURL(NULL, *EditorEngine->BuildPlayWorldURL(*PIEMapName, Params.bStartInSpectatorMode, ExtraURLOptions), TRAVEL_Absolute);

	if (CVS.OwningGameInstance && PreviewWorld->WorldType == EWorldType::GamePreview)
	{
		PreviewWorld->SetGameInstance(CVS.OwningGameInstance);

		FWorldContext& PreviewWorldContext = GEngine->GetWorldContextFromWorldChecked(PreviewWorld);
		PreviewWorldContext.OwningGameInstance = CVS.OwningGameInstance;
		PreviewWorldContext.GameViewport = CVS.OwningGameInstance->GetGameViewportClient();
		PreviewWorldContext.AddRef(PreviewWorld);
		
		//PreviewWorldContext.PIEInstance =

		if (CVS.DefaultGameMode)
		{
			PreviewWorld->SetGameMode(URL);

			AGameModeBase* Mode = PreviewWorld->GetAuthGameMode<AGameModeBase>();
			ensure(Mode);
		}
	}

	PreviewWorld->InitializeActorsForPlay(URL);

	if (CVS.bDefaultLighting)
	{
		DirectionalLight = NewObject<UDirectionalLightComponent>(GetTransientPackage(), NAME_None, RF_Transient);
		DirectionalLight->Intensity = CVS.LightBrightness;
		DirectionalLight->LightColor = FColor::White;
		DirectionalLight->bTransmission = true;
		AddComponent(DirectionalLight, FTransform(CVS.LightRotation));

		SkyLight = NewObject<USkyLightComponent>(GetTransientPackage(), NAME_None, RF_Transient);
		SkyLight->bLowerHemisphereIsBlack = false;
		SkyLight->SourceType = ESkyLightSourceType::SLS_SpecifiedCubemap;
		SkyLight->Intensity = CVS.SkyBrightness;
		SkyLight->Mobility = EComponentMobility::Movable;
		AddComponent(SkyLight, FTransform::Identity);

		LineBatcher = NewObject<ULineBatchComponent>(GetTransientPackage());
		LineBatcher->bCalculateAccurateBounds = false;
		AddComponent(LineBatcher, FTransform::Identity);
	}

	FCoreDelegates::OnEnginePreExit.AddRaw(this, &FPreviewScene::Uninitialize);
}

FPreviewScene::~FPreviewScene()
{
	FCoreDelegates::OnEnginePreExit.RemoveAll(this);
	Uninitialize();
}

void FPreviewScene::Uninitialize()
{
	// Stop any audio components playing in this scene
	if (GEngine)
	{
		UWorld* World = GetWorld();
		if (World)
		{
			if (FAudioDeviceHandle AudioDevice = World->GetAudioDevice())
			{
				AudioDevice->Flush(GetWorld(), false);
			}
		}
	}

	// Remove all the attached components
	for (int32 ComponentIndex = 0; ComponentIndex < Components.Num(); ComponentIndex++)
	{
		UActorComponent* Component = Components[ComponentIndex];

		if (bForceAllUsedMipsResident)
		{
			// Remove the mip streaming override on the mesh to be removed
			UMeshComponent* pMesh = Cast<UMeshComponent>(Component);
			if (pMesh != NULL)
			{
				pMesh->SetTextureForceResidentFlag(false);
			}
		}

#if WITH_EDITOR
		if (UMeshComponent* pMesh = Cast<UMeshComponent>(Component))
		{
			pMesh->SetMarkTextureAsEditorStreamingPool(false);
		}
#endif

		Component->UnregisterComponent();
	}

	// Uninitialize can get called from destructor or FCoreDelegates::OnPreExit (or both)
	// so make sure we empty Components and set PreviewWorld to nullptr
	Components.Empty();
		
	UWorld* LocalPreviewWorld = PreviewWorld;
	PreviewWorld = nullptr;

	// The world may be released by now.
	if (LocalPreviewWorld && GEngine)
	{
		LocalPreviewWorld->CleanupWorld();
		GEngine->DestroyWorldContext(LocalPreviewWorld);
		// Release PhysicsScene for fixing big fbx importing bug
		LocalPreviewWorld->ReleasePhysicsScene();

		if (PreviewScene::Private::bForceGCOnPreviewSceneExit)
		{
			// The preview world is a heavy-weight object and may hold a significant amount of resources,
			// including various GPU render targets and buffers required for rendering the scene.
			// Since UWorld is garbage-collected, this memory may not be cleaned for an indeterminate amount of time.
			// By forcing garbage collection explicitly, we allow memory to be reused immediately.
			GEngine->ForceGarbageCollection(true /*bFullPurge*/);
		}
	}
}

void FPreviewScene::AddComponent(UActorComponent* Component,const FTransform& LocalToWorld, bool bAttachToRoot /*= false*/)
{
	Components.AddUnique(Component);

	USceneComponent* SceneComp = Cast<USceneComponent>(Component);
	if(SceneComp && SceneComp->GetAttachParent() == NULL)
	{
		SceneComp->SetRelativeTransform(LocalToWorld);
	}

	Component->RegisterComponentWithWorld(GetWorld());

	if (bForceAllUsedMipsResident)
	{
		// Add a mip streaming override to the new mesh
		UMeshComponent* pMesh = Cast<UMeshComponent>(Component);
		if (pMesh != NULL)
		{
			pMesh->SetTextureForceResidentFlag(true);
		}
	}

#if WITH_EDITOR
	if (UMeshComponent* pMesh = Cast<UMeshComponent>(Component))
	{
		pMesh->SetMarkTextureAsEditorStreamingPool(true);
	}
#endif

	{
		UStaticMeshComponent* pStaticMesh = Cast<UStaticMeshComponent>(Component);
		if(pStaticMesh != nullptr)
		{
			pStaticMesh->bEvaluateWorldPositionOffset = true;
			pStaticMesh->bEvaluateWorldPositionOffsetInRayTracing = true;
		}
	}

	GetScene()->UpdateSpeedTreeWind(0.0);
}

void FPreviewScene::RemoveComponent(UActorComponent* Component)
{
	Component->UnregisterComponent();
	Components.Remove(Component);

	if (bForceAllUsedMipsResident)
	{
		// Remove the mip streaming override on the old mesh
		UMeshComponent* pMesh = Cast<UMeshComponent>(Component);
		if (pMesh != NULL)
		{
			pMesh->SetTextureForceResidentFlag(false);
		}
	}
}

void FPreviewScene::AddReferencedObjects( FReferenceCollector& Collector )
{
	Collector.AddReferencedObjects( Components );
	Collector.AddReferencedObject( PreviewWorld );
}

FString FPreviewScene::GetReferencerName() const
{
	return TEXT("FPreviewScene");
}

void FPreviewScene::UpdateCaptureContents()
{
	// This function is called from FAdvancedPreviewScene::Tick, FBlueprintEditor::Tick, and FThumbnailPreviewScene::Tick,
	// so assume we are inside a Tick function.
	const bool bInsideTick = true;

	USkyLightComponent::UpdateSkyCaptureContents(PreviewWorld);
	UReflectionCaptureComponent::UpdateReflectionCaptureContents(PreviewWorld, nullptr, false, false, bInsideTick);
}

void FPreviewScene::ClearLineBatcher()
{
	if (LineBatcher != NULL)
	{
		LineBatcher->Flush();
	}
}

/** Accessor for finding the current direction of the preview scene's DirectionalLight. */
FRotator FPreviewScene::GetLightDirection()
{
	if (DirectionalLight != NULL)
	{
		return DirectionalLight->GetComponentTransform().GetUnitAxis( EAxis::X ).Rotation();
	}

	return FRotator::ZeroRotator;
}

/** Function for modifying the current direction of the preview scene's DirectionalLight. */
void FPreviewScene::SetLightDirection(const FRotator& InLightDir)
{
	if (DirectionalLight != NULL)
	{
#if WITH_EDITOR
		DirectionalLight->PreEditChange(NULL);
#endif // WITH_EDITOR
		DirectionalLight->SetAbsolute(true, true, true);
		DirectionalLight->SetRelativeRotation(InLightDir);
#if WITH_EDITOR
		DirectionalLight->PostEditChange();
#endif // WITH_EDITOR
	}
}

void FPreviewScene::SetLightBrightness(float LightBrightness)
{
	if (DirectionalLight != NULL)
	{
#if WITH_EDITOR
		DirectionalLight->PreEditChange(NULL);
#endif // WITH_EDITOR
		DirectionalLight->Intensity = LightBrightness;
#if WITH_EDITOR
		DirectionalLight->PostEditChange();
#endif // WITH_EDITOR
	}
}

void FPreviewScene::SetLightColor(const FColor& LightColor)
{
	if (DirectionalLight != NULL)
	{
#if WITH_EDITOR
		DirectionalLight->PreEditChange(NULL);
#endif // WITH_EDITOR
		DirectionalLight->LightColor = LightColor;
#if WITH_EDITOR
		DirectionalLight->PostEditChange();
#endif // WITH_EDITOR
	}
}

void FPreviewScene::SetSkyBrightness(float SkyBrightness)
{
	if (SkyLight != NULL)
	{
		SkyLight->SetIntensity(SkyBrightness);
	}
}

void FPreviewScene::SetSkyCubemap(UTextureCube* Cubemap)
{
	if (SkyLight != NULL)
	{
		SkyLight->SetCubemap(Cubemap);
	}
}

void FPreviewScene::LoadSettings(const TCHAR* Section)
{
	FRotator LightDir;
	if ( GConfig->GetRotator( Section, TEXT("LightDir"), LightDir, GEditorPerProjectIni ) )
	{
		SetLightDirection( LightDir );
	}
}

void FPreviewScene::SaveSettings(const TCHAR* Section)
{
	GConfig->SetRotator( Section, TEXT("LightDir"), GetLightDirection(), GEditorPerProjectIni );
}

FLinearColor FPreviewScene::GetBackgroundColor() const
{
	return FLinearColor(0.05f, 0.05f, 0.05f);
}
