// Copyright Epic Games, Inc. All Rights Reserved.

#include "Engine/PrimaryAssetLabel.h"
#include "AssetRegistry/AssetData.h"
#include "AssetRegistry/IAssetRegistry.h"
#include "Engine/DataAsset.h"
#include "Misc/PackageName.h"
#include "Engine/AssetManager.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PrimaryAssetLabel)

#if WITH_EDITOR
#include "ICollectionContainer.h"
#include "ICollectionManager.h"
#include "CollectionManagerModule.h"
#endif

const FName UPrimaryAssetLabel::DirectoryBundle = FName("Directory");
const FName UPrimaryAssetLabel::CollectionBundle = FName("Collection");

UPrimaryAssetLabel::UPrimaryAssetLabel()
{
	bLabelAssetsInMyDirectory = false;
	bIsRuntimeLabel = false;
	bIncludeRedirectors = true;

	// By default have low priority and don't recurse
	Rules.bApplyRecursively = false;
	Rules.Priority = 0;
}

#if WITH_EDITORONLY_DATA
void UPrimaryAssetLabel::UpdateAssetBundleData()
{
	Super::UpdateAssetBundleData();

	if (!UAssetManager::IsInitialized())
	{
		return;
	}

	UAssetManager& Manager = UAssetManager::Get();
	IAssetRegistry& AssetRegistry = Manager.GetAssetRegistry();

	if (bLabelAssetsInMyDirectory)
	{
		FName PackagePath = FName(*FPackageName::GetLongPackagePath(GetOutermost()->GetName()));

		TArray<FAssetData> DirectoryAssets;
		FARFilter Filter;
		Filter.bRecursivePaths = true;
		Filter.PackagePaths.Add(PackagePath);
		Filter.bIncludeOnlyOnDiskAssets = IsRunningCookCommandlet();	// Enumerate only on-disk packages when cooking to avoid indeterministic results
		AssetRegistry.GetAssets(Filter, DirectoryAssets);

		TArray<FTopLevelAssetPath> NewPaths;

		for (const FAssetData& AssetData : DirectoryAssets)
		{
			FSoftObjectPath AssetRef = Manager.GetAssetPathForData(AssetData);

			if (!AssetRef.IsNull() && (bIncludeRedirectors || !AssetData.IsRedirector()))
			{
				NewPaths.Add(AssetRef.GetAssetPath());
			}
		}

		// Fast set, destroys NewPaths
		AssetBundleData.SetBundleAssets(DirectoryBundle, MoveTemp(NewPaths));
	}

	if (AssetCollection.CollectionName != NAME_None)
	{
		TArray<FTopLevelAssetPath> NewPaths;
		TArray<FSoftObjectPath> CollectionAssets;
		const TSharedRef<ICollectionContainer>& CollectionContainer = FCollectionManagerModule::GetModule().Get().GetProjectCollectionContainer();
		CollectionContainer->GetAssetsInCollection(AssetCollection.CollectionName, ECollectionShareType::CST_All, CollectionAssets);
		for (int32 Index = 0; Index < CollectionAssets.Num(); ++Index)
		{
			FAssetData FoundAsset = Manager.GetAssetRegistry().GetAssetByObjectPath(CollectionAssets[Index]);
			FSoftObjectPath AssetRef = Manager.GetAssetPathForData(FoundAsset);

			if (!AssetRef.IsNull() && (bIncludeRedirectors || !FoundAsset.IsRedirector()))
			{
				NewPaths.Add(AssetRef.GetAssetPath());
			}
		}

		// Fast set, destroys NewPaths
		AssetBundleData.SetBundleAssets(CollectionBundle, MoveTemp(NewPaths));
	}
	
	// Update rules
	FPrimaryAssetId PrimaryAssetId = GetPrimaryAssetId();
	Manager.SetPrimaryAssetRules(PrimaryAssetId, Rules);
}
#endif

