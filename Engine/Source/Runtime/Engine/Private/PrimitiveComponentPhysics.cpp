// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/PrimitiveComponent.h"
#include "AI/NavigationSystemBase.h"
#include "Collision/CollisionConversions.h"
#include "Engine/OverlapResult.h"
#include "EngineLogs.h"
#include "Logging/MessageLog.h"
#include "Physics/Experimental/PhysScene_Chaos.h"
#include "PhysicsEngine/BodySetup.h"
#include "DrawDebugHelpers.h"
#include "PhysicsReplication.h"
#include "UObject/UObjectThreadContext.h"
#include "PhysicsEngine/PhysicsObjectExternalInterface.h"
#include "PhysicsProxy/SingleParticlePhysicsProxy.h"
#include "PhysicalMaterials/PhysicalMaterial.h"
#include "Components/SkeletalMeshComponent.h"
#include "Materials/MaterialInterface.h"
#include "Engine/SkeletalMeshSocket.h"

//////////////// PRIMITIVECOMPONENT ///////////////

#define LOCTEXT_NAMESPACE "PrimitiveComponent"

DECLARE_CYCLE_STAT(TEXT("WeldPhysics"), STAT_WeldPhysics, STATGROUP_Physics);
DECLARE_CYCLE_STAT(TEXT("UnweldPhysics"), STAT_UnweldPhysics, STATGROUP_Physics);
DECLARE_CYCLE_STAT_WITH_FLAGS(TEXT("PrimComp SetCollisionProfileName"), STAT_PrimComp_SetCollisionProfileName, STATGROUP_Physics, EStatFlags::Verbose);


#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	#define WarnInvalidPhysicsOperations(Text, BodyInstance, BoneName)  { static const FText _WarnText(Text); WarnInvalidPhysicsOperations_Internal(_WarnText, BodyInstance, BoneName); }
#else
	#define WarnInvalidPhysicsOperations(Text, BodyInstance, BoneName)
#endif

namespace PrimitiveComponentCVars
{
	bool bReplicatePhysicsObject = 1;
	static FAutoConsoleVariableRef CVarReplicatePhysicsObject(
		TEXT("p.PrimitiveComponent.ReplicatePhysicsObject"),
		bReplicatePhysicsObject,
		TEXT("When a primitive component has no BodyInstance, allow replication based on PhysicsObject\n"),
		ECVF_Default);
}

void UPrimitiveComponent::SetRigidBodyReplicatedTarget(FRigidBodyState& UpdatedState, FName BoneName, int32 ServerFrame, int32 ServerHandle)
{
	if (!CanBeUsedInPhysicsReplication(BoneName))
	{
		return;
	}

	if (UWorld* World = GetWorld())
	{
		if (FPhysScene* PhysScene = World->GetPhysicsScene())
		{
			if (IPhysicsReplication* PhysicsReplication = PhysScene->GetPhysicsReplication())
			{
				// If we are not allowed to replicate physics objects,
				// don't set replicated target unless we have a BodyInstance.
				if (PrimitiveComponentCVars::bReplicatePhysicsObject == false)
				{
					FBodyInstance* BI = GetBodyInstance(BoneName);
					if (BI == nullptr || !BI->IsValidBodyInstance())
					{
						return;
					}
				}
				
				PhysicsReplication->SetReplicatedTarget(this, BoneName, UpdatedState, ServerFrame);
			}
		}
	}
}

void UPrimitiveComponent::RegisterAsFocalPointInPhysicsReplicationLOD() const
{
	if (UWorld* World = GetWorld())
	{
		if (FPhysScene* PhysScene = World->GetPhysicsScene())
		{
			if (IPhysicsReplicationLOD* PhysicsReplicationLOD = PhysScene->GetPhysicsReplicationLOD())
			{
				PhysicsReplicationLOD->RegisterFocalPoint_External(this);
			}
		}
	}
}

void UPrimitiveComponent::UnregisterAsFocalPointInPhysicsReplicationLOD() const
{
	if (UWorld* World = GetWorld())
	{
		if (FPhysScene* PhysScene = World->GetPhysicsScene())
		{
			if (IPhysicsReplicationLOD* PhysicsReplicationLOD = PhysScene->GetPhysicsReplicationLOD())
			{
				PhysicsReplicationLOD->UnregisterFocalPoint_External(this);
			}
		}
	}
}

bool UPrimitiveComponent::GetRigidBodyState(FRigidBodyState& OutState, FName BoneName)
{
	// If we have a BodyInstance, use it
	//
	// TODO: Remove this code path
	FBodyInstance* BI = GetBodyInstance(BoneName);
	if (BI)
	{
		return BI->GetRigidBodyState(OutState);
	}

	// If we don't, get data from the physics object.
	//
	// TODO: Add support for multiple physics objects
	if (PrimitiveComponentCVars::bReplicatePhysicsObject)
	{
		if (Chaos::FPhysicsObject* PhysicsObject = GetPhysicsObjectByName(BoneName))
		{
			FLockedReadPhysicsObjectExternalInterface Interface = FPhysicsObjectExternalInterface::LockRead(PhysicsObject);
			const FTransform Transform = Interface->GetTransform(PhysicsObject);
			OutState.Position = Transform.GetLocation();
			OutState.Quaternion = Transform.GetRotation();
			OutState.LinVel = Interface->GetV(PhysicsObject);
			OutState.AngVel = FMath::RadiansToDegrees(Interface->GetW(PhysicsObject));
			OutState.Flags = (Interface->AreAllSleeping({ PhysicsObject }) ? ERigidBodyFlags::Sleeping : ERigidBodyFlags::None);
			return true;
		}
	}

	return false;
}

const struct FWalkableSlopeOverride& UPrimitiveComponent::GetWalkableSlopeOverride() const
{
	return BodyInstance.GetWalkableSlopeOverride();
}

void UPrimitiveComponent::SetWalkableSlopeOverride(const FWalkableSlopeOverride& NewOverride)
{
	BodyInstance.SetWalkableSlopeOverride(NewOverride);
}

void UPrimitiveComponent::WarnInvalidPhysicsOperations_Internal(const FText& ActionText, const FBodyInstance* BI, FName BoneName) const
{
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	if (!CheckStaticMobilityAndWarn(ActionText))	//all physics operations require non-static mobility
	{
		if (BI)
		{
			ECollisionEnabled::Type CollisionEnabled = BI->GetCollisionEnabled();

			FString Identity = GetReadableName();
			if(BoneName != NAME_None)
			{
				Identity += " (bone:" + BoneName.ToString() + ")";
			}

			if(BI->bSimulatePhysics == false)	//some require to be simulating too
			{
				
				FMessageLog("PIE").Warning(FText::Format(LOCTEXT("InvalidPhysicsOperationSimulatePhysics", "{0} has to have 'Simulate Physics' enabled if you'd like to {1}. "), FText::FromString(Identity), ActionText));
			}
			else if (CollisionEnabled == ECollisionEnabled::NoCollision || CollisionEnabled == ECollisionEnabled::QueryOnly)	//shapes need to be simulating
			{
				FMessageLog("PIE").Warning(FText::Format(LOCTEXT("InvalidPhysicsOperationCollisionDisabled", "{0} has to have 'CollisionEnabled' set to 'Query and Physics' or 'Physics only' if you'd like to {1}. "), FText::FromString(Identity), ActionText));
			}
		}
	}
#endif
}


void UPrimitiveComponent::SetSimulatePhysics(bool bSimulate)
{
	BodyInstance.SetInstanceSimulatePhysics(bSimulate);
}

void UPrimitiveComponent::SetStaticWhenNotMoveable(bool bInStaticWhenNotMoveable)
{
	if (bStaticWhenNotMoveable != bInStaticWhenNotMoveable)
	{
		bStaticWhenNotMoveable = bInStaticWhenNotMoveable;
		RecreatePhysicsState();
	}
}

void UPrimitiveComponent::SetConstraintMode(EDOFMode::Type ConstraintMode)
{
	FBodyInstance * RootBI = GetBodyInstance(NAME_None, false);

	if (RootBI == NULL || !IsValidChecked(this))
	{
		return;
	}

	RootBI->SetDOFLock(ConstraintMode);
}

void UPrimitiveComponent::AddImpulse(FVector Impulse, FName BoneName, bool bVelChange)
{
	if (FBodyInstance* BI = GetBodyInstance(BoneName))
	{
		WarnInvalidPhysicsOperations(LOCTEXT("AddImpulse", "AddImpulse"), BI, BoneName);
		BI->AddImpulse(Impulse, bVelChange);
	}
	else if (BoneName == NAME_None)
	{
		TArray<Chaos::FPhysicsObjectHandle> PhysicsObjects = GetAllPhysicsObjects();
		FLockedWritePhysicsObjectExternalInterface Interface = FPhysicsObjectExternalInterface::LockWrite(PhysicsObjects);
		
		PhysicsObjects = PhysicsObjects.FilterByPredicate(
			[&Interface](Chaos::FPhysicsObject* Object) {
			return !Interface->AreAllDisabled({ &Object, 1 });
		}
		);
		Interface->SetLinearImpulseVelocity(PhysicsObjects, Impulse, bVelChange);
	}
	else if (Chaos::FPhysicsObject* Object = GetPhysicsObjectByName(BoneName))
	{
		FLockedWritePhysicsObjectExternalInterface Interface = FPhysicsObjectExternalInterface::LockWrite({ &Object, 1 });
		Interface->SetLinearImpulseVelocity({ &Object, 1 }, Impulse, bVelChange);
	}

}

void UPrimitiveComponent::AddAngularImpulseInRadians(FVector Impulse, FName BoneName, bool bVelChange)
{
	if (FBodyInstance* BI = GetBodyInstance(BoneName))
	{
		WarnInvalidPhysicsOperations(LOCTEXT("AddAngularImpulse", "AddAngularImpulse"), BI, BoneName);
		BI->AddAngularImpulseInRadians(Impulse, bVelChange);
	}
}

void UPrimitiveComponent::AddImpulseAtLocation(FVector Impulse, FVector Location, FName BoneName)
{
	if (FBodyInstance* BI = GetBodyInstance(BoneName))
	{
		WarnInvalidPhysicsOperations(LOCTEXT("AddImpulseAtLocation", "AddImpulseAtLocation"), BI, BoneName);
		BI->AddImpulseAtPosition(Impulse, Location);
	}
}

void UPrimitiveComponent::AddVelocityChangeImpulseAtLocation(FVector Impulse, FVector Location, FName BoneName)
{
	if (FBodyInstance* BI = GetBodyInstance(BoneName))
	{
		WarnInvalidPhysicsOperations(LOCTEXT("AddImpulseAtLocation", "AddImpulseAtLocation"), BI, BoneName);
		BI->AddVelocityChangeImpulseAtLocation(Impulse, Location);
	}
}

void UPrimitiveComponent::AddRadialImpulse(FVector Origin, float Radius, float Strength, ERadialImpulseFalloff Falloff, bool bVelChange)
{
	if(bIgnoreRadialImpulse)
	{
		return;
	}

	FBodyInstance* BI = GetBodyInstance();
	if (BI)
	{
		BI->AddRadialImpulseToBody(Origin, Radius, Strength, Falloff, bVelChange);
	}
	else
	{
		TArray<Chaos::FPhysicsObjectHandle> PhysicsObjects = GetAllPhysicsObjects();
		FLockedWritePhysicsObjectExternalInterface Interface = FPhysicsObjectExternalInterface::LockWrite(PhysicsObjects);

		PhysicsObjects = PhysicsObjects.FilterByPredicate(
			[&Interface](Chaos::FPhysicsObject* Object) {
				return !Interface->AreAllDisabled({ &Object, 1 });
			}
		);

		Interface->AddRadialImpulse(PhysicsObjects, Origin, Radius, Strength, Falloff, /*bApplyStrain*/true, /*bInvalidate*/true, bVelChange, /*MinValue*/0.f, /*MaxValue*/1.f);
	}
}


void UPrimitiveComponent::AddForce(FVector Force, FName BoneName, bool bAccelChange)
{
	if (FBodyInstance* BI = GetBodyInstance(BoneName))
	{
		WarnInvalidPhysicsOperations(LOCTEXT("AddForce", "AddForce"), BI, BoneName);
		BI->AddForce(Force, true, bAccelChange);
	}
	else if (BoneName == NAME_None)
	{
		TArray<Chaos::FPhysicsObjectHandle> PhysicsObjects = GetAllPhysicsObjects();
		FLockedWritePhysicsObjectExternalInterface Interface = FPhysicsObjectExternalInterface::LockWrite(PhysicsObjects);

		PhysicsObjects = PhysicsObjects.FilterByPredicate(
			[&Interface](Chaos::FPhysicsObject* Object) {
				return !Interface->AreAllDisabled({ &Object, 1 });
			}
		);

		if (bAccelChange)
		{
			const float Mass = Interface->GetMass(PhysicsObjects);
			Interface->AddForce(PhysicsObjects, Force * Mass, /*bInvalidate*/true);
		}
		else
		{
			Interface->AddForce(PhysicsObjects, Force, /*bInvalidate*/true);
		}
	}
	else if (Chaos::FPhysicsObject* Object = GetPhysicsObjectByName(BoneName))
	{
		FLockedWritePhysicsObjectExternalInterface Interface = FPhysicsObjectExternalInterface::LockWrite({ &Object, 1 });
		if (bAccelChange)
		{
			const float Mass = Interface->GetMass({ &Object, 1 });
			Interface->AddForce({ &Object, 1 }, Force * Mass, /*bInvalidate*/true);
		}
		else
		{
			Interface->AddForce({ &Object, 1 }, Force, /*bInvalidate*/true);
		}
	}
}

void UPrimitiveComponent::AddForceAtLocation(FVector Force, FVector Location, FName BoneName)
{
	if (FBodyInstance* BI = GetBodyInstance(BoneName))
	{
		WarnInvalidPhysicsOperations(LOCTEXT("AddForceAtLocation", "AddForceAtLocation"), BI, BoneName);
		BI->AddForceAtPosition(Force, Location);
	}
}

void UPrimitiveComponent::AddForceAtLocationLocal(FVector Force, FVector Location, FName BoneName)
{
	if (FBodyInstance* BI = GetBodyInstance(BoneName))
	{
		WarnInvalidPhysicsOperations(LOCTEXT("AddForceAtLocationLocal", "AddForceAtLocationLocal"), BI, BoneName);
		BI->AddForceAtPosition(Force, Location, /* bAllowSubstepping = */ true, /* bIsForceLocal = */ true);
	}
}

void UPrimitiveComponent::AddRadialForce(FVector Origin, float Radius, float Strength, ERadialImpulseFalloff Falloff, bool bAccelChange)
{
	if(bIgnoreRadialForce)
	{
		return;
	}

	FBodyInstance* BI = GetBodyInstance();
	if (BI)
	{
		BI->AddRadialForceToBody(Origin, Radius, Strength, Falloff, bAccelChange);
	}
}

void UPrimitiveComponent::AddTorqueInRadians(FVector Torque, FName BoneName, bool bAccelChange)
{
	if (FBodyInstance* BI = GetBodyInstance(BoneName))
	{
		WarnInvalidPhysicsOperations(LOCTEXT("AddTorque", "AddTorque"), BI, BoneName);
		BI->AddTorqueInRadians(Torque, true, bAccelChange);
	}
	else if (BoneName == NAME_None)
	{
		TArray<Chaos::FPhysicsObjectHandle> PhysicsObjects = GetAllPhysicsObjects();
		FLockedWritePhysicsObjectExternalInterface Interface = FPhysicsObjectExternalInterface::LockWrite(PhysicsObjects);

		PhysicsObjects = PhysicsObjects.FilterByPredicate(
			[&Interface](Chaos::FPhysicsObject* Object) {
				return !Interface->AreAllDisabled({ &Object, 1 });
			}
		);
		Interface->AddTorque(PhysicsObjects, Torque, /*bInvalidate*/true, bAccelChange);
	}
	else if (Chaos::FPhysicsObject* Object = GetPhysicsObjectByName(BoneName))
	{
		FLockedWritePhysicsObjectExternalInterface Interface = FPhysicsObjectExternalInterface::LockWrite({ &Object, 1 });
		Interface->AddTorque({ &Object, 1 }, Torque, /*bInvalidate*/true, bAccelChange);
	}
}

void UPrimitiveComponent::SetPhysicsLinearVelocity(FVector NewVel, bool bAddToCurrent, FName BoneName)
{
	if (FBodyInstance* BI = GetBodyInstance(BoneName))
	{
		WarnInvalidPhysicsOperations(LOCTEXT("SetPhysicsLinearVelocity", "SetPhysicsLinearVelocity"), nullptr, BoneName);
		BI->SetLinearVelocity(NewVel, bAddToCurrent);
	}
	else if (BoneName == NAME_None)
	{
		TArray<Chaos::FPhysicsObjectHandle> PhysicsObjects = GetAllPhysicsObjects();
		FLockedWritePhysicsObjectExternalInterface Interface = FPhysicsObjectExternalInterface::LockWrite(PhysicsObjects);

		PhysicsObjects = PhysicsObjects.FilterByPredicate(
			[&Interface](Chaos::FPhysicsObject* Object) {
				return !Interface->AreAllDisabled({ &Object, 1 });
			}
		);
		Interface->SetLinearVelocity(PhysicsObjects, NewVel, bAddToCurrent);
	}
	else if (Chaos::FPhysicsObject* Object = GetPhysicsObjectByName(BoneName))
	{
		FLockedWritePhysicsObjectExternalInterface Interface = FPhysicsObjectExternalInterface::LockWrite({ &Object, 1 });
		Interface->SetLinearVelocity({ &Object, 1 }, NewVel, bAddToCurrent);
	}
}

FVector UPrimitiveComponent::GetPhysicsLinearVelocity(FName BoneName) const
{
	if (FBodyInstance* BI = GetBodyInstance(BoneName))
	{
		return BI->GetUnrealWorldVelocity();
	}
	else if (Chaos::FConstPhysicsObjectHandle Handle = GetPhysicsObjectByName(BoneName))
	{
		FLockedReadPhysicsObjectExternalInterface Interface = FPhysicsObjectExternalInterface::LockRead(Handle);
		return Interface->GetV(Handle);
	}
	return FVector(0,0,0);
}

FVector UPrimitiveComponent::GetPhysicsLinearVelocityAtPoint(FVector Point, FName BoneName) const
{
	if (FBodyInstance* BI = GetBodyInstance(BoneName))
	{
		return BI->GetUnrealWorldVelocityAtPoint(Point);
	}
	else if (Chaos::FConstPhysicsObjectHandle Handle = GetPhysicsObjectByName(BoneName))
	{
		FLockedReadPhysicsObjectExternalInterface Interface = FPhysicsObjectExternalInterface::LockRead(Handle);
		return Interface->GetVAtPoint(Handle, Point);
	}
	return FVector(0, 0, 0);
}

void UPrimitiveComponent::SetAllPhysicsLinearVelocity(FVector NewVel,bool bAddToCurrent)
{
	SetPhysicsLinearVelocity(NewVel, bAddToCurrent, NAME_None);
}

void UPrimitiveComponent::SetPhysicsAngularVelocityInRadians(FVector NewAngVel, bool bAddToCurrent, FName BoneName)
{
	if (FBodyInstance* BI = GetBodyInstance(BoneName))
	{
		WarnInvalidPhysicsOperations(LOCTEXT("SetPhysicsAngularVelocity", "SetPhysicsAngularVelocity"), nullptr, BoneName);
		BI->SetAngularVelocityInRadians(NewAngVel, bAddToCurrent);
	}
	else if (BoneName == NAME_None)
	{
		TArray<Chaos::FPhysicsObjectHandle> PhysicsObjects = GetAllPhysicsObjects();
		FLockedWritePhysicsObjectExternalInterface Interface = FPhysicsObjectExternalInterface::LockWrite(PhysicsObjects);

		PhysicsObjects = PhysicsObjects.FilterByPredicate(
			[&Interface](Chaos::FPhysicsObject* Object) {
				return !Interface->AreAllDisabled({ &Object, 1 });
			}
		);
		Interface->SetAngularVelocityInRadians(PhysicsObjects, NewAngVel, bAddToCurrent);
	}
	else if (Chaos::FPhysicsObject* Object = GetPhysicsObjectByName(BoneName))
	{
		FLockedWritePhysicsObjectExternalInterface Interface = FPhysicsObjectExternalInterface::LockWrite({ &Object, 1 });
		Interface->SetAngularVelocityInRadians({ &Object, 1 }, NewAngVel, bAddToCurrent);
	}
}

void UPrimitiveComponent::SetPhysicsMaxAngularVelocityInRadians(float NewMaxAngVel, bool bAddToCurrent, FName BoneName)
{
	if (FBodyInstance* BI = GetBodyInstance(BoneName))
	{
		WarnInvalidPhysicsOperations(LOCTEXT("SetPhysicsMaxAngularVelocity", "SetPhysicsMaxAngularVelocity"), nullptr, BoneName);
		BI->SetMaxAngularVelocityInRadians(NewMaxAngVel, bAddToCurrent);
	}
}

FVector UPrimitiveComponent::GetPhysicsAngularVelocityInRadians(FName BoneName) const
{
	if (FBodyInstance* const BI = GetBodyInstance(BoneName))
	{
		return BI->GetUnrealWorldAngularVelocityInRadians();
	}
	else if (Chaos::FConstPhysicsObjectHandle Handle = GetPhysicsObjectByName(BoneName))
	{
		FLockedReadPhysicsObjectExternalInterface Interface = FPhysicsObjectExternalInterface::LockRead(Handle);
		return Interface->GetW(Handle);
	}
	return FVector(0,0,0);
}

FVector UPrimitiveComponent::GetCenterOfMass(FName BoneName) const
{
	if (FBodyInstance* ComponentBodyInstance = GetBodyInstance(BoneName))
	{
		return ComponentBodyInstance->GetCOMPosition();
	}
	else if (Chaos::FConstPhysicsObjectHandle Handle = GetPhysicsObjectByName(BoneName))
	{
		FLockedReadPhysicsObjectExternalInterface Interface = FPhysicsObjectExternalInterface::LockRead(Handle);
		return Interface->GetWorldCoM(Handle);
	}

	return FVector::ZeroVector;
}

void UPrimitiveComponent::SetCenterOfMass(FVector CenterOfMassOffset, FName BoneName)
{
	if (FBodyInstance* ComponentBodyInstance = GetBodyInstance(BoneName))
	{
		WarnInvalidPhysicsOperations(LOCTEXT("SetCenterOfMass", "SetCenterOfMass"), nullptr, BoneName);
		ComponentBodyInstance->COMNudge = CenterOfMassOffset;
		ComponentBodyInstance->UpdateMassProperties();
	}
}

void UPrimitiveComponent::SetAllPhysicsAngularVelocityInRadians(FVector const& NewAngVel, bool bAddToCurrent)
{
	SetPhysicsAngularVelocityInRadians(NewAngVel, bAddToCurrent, NAME_None); 
}


void UPrimitiveComponent::SetAllPhysicsPosition(FVector NewPos)
{
	SetWorldLocation(NewPos);
}


void UPrimitiveComponent::SetAllPhysicsRotation(FRotator NewRot)
{
	SetWorldRotation(NewRot);
}

void UPrimitiveComponent::SetAllPhysicsRotation(const FQuat& NewRot)
{
	SetWorldRotation(NewRot);
}

void UPrimitiveComponent::WakeRigidBody(FName BoneName)
{
	FBodyInstance* BI = GetBodyInstance(BoneName);
	if(BI)
	{
		BI->WakeInstance();
	}
	else if (BoneName == NAME_None)
	{
		TArray<Chaos::FPhysicsObject*> PhysicsObjects = GetAllPhysicsObjects();
		FLockedWritePhysicsObjectExternalInterface Interface = FPhysicsObjectExternalInterface::LockWrite(PhysicsObjects);
		
		PhysicsObjects = PhysicsObjects.FilterByPredicate(
			[&Interface](Chaos::FPhysicsObject* Object) {
				return !Interface->AreAllDisabled({ &Object, 1 });
			}
		);
		Interface->WakeUp(PhysicsObjects);
	}
	else if (Chaos::FPhysicsObject* Object = GetPhysicsObjectByName(BoneName))
	{
		FLockedWritePhysicsObjectExternalInterface Interface = FPhysicsObjectExternalInterface::LockWrite({ &Object, 1 });
		Interface->WakeUp({ &Object, 1 });
	}
}

void UPrimitiveComponent::WakeAllRigidBodies()
{
	WakeRigidBody(NAME_None);
}

void UPrimitiveComponent::SetEnableGravity(bool bGravityEnabled)
{
	FBodyInstance* BI = GetBodyInstance();
	if (BI)
	{
		BI->SetEnableGravity(bGravityEnabled);
	}
}

bool UPrimitiveComponent::IsGravityEnabled() const
{
	FBodyInstance* BI = GetBodyInstance();
	if (BI)
	{
		return BI->bEnableGravity;
	}

	return false;
}

void UPrimitiveComponent::SetUpdateKinematicFromSimulation(bool bUpdateKinematicFromSimulation)
{
	FBodyInstance* BI = GetBodyInstance();
	if (BI)
	{
		BI->SetUpdateKinematicFromSimulation(bUpdateKinematicFromSimulation);
	}
}

bool UPrimitiveComponent::GetUpdateKinematicFromSimulation() const
{
	FBodyInstance* BI = GetBodyInstance();
	if (BI)
	{
		return BI->bUpdateKinematicFromSimulation;
	}

	return false;
}

void UPrimitiveComponent::SetGyroscopicTorqueEnabled(bool bEnable)
{
	FBodyInstance* BI = GetBodyInstance();
	if (BI)
	{
		BI->SetGyroscopicTorqueEnabled(bEnable);
	}
}

bool UPrimitiveComponent::GetGyroscopicTorqueEnabled() const
{
	FBodyInstance* BI = GetBodyInstance();
	if (BI)
	{
		return BI->bGyroscopicTorqueEnabled;
	}
	return false;
}

void UPrimitiveComponent::SetLinearDamping(float InDamping)
{
	FBodyInstance* BI = GetBodyInstance();
	if (BI)
	{
		BI->LinearDamping = InDamping;
		BI->UpdateDampingProperties();
	}

	else if (Chaos::FPhysicsObject* Object = GetPhysicsObjectByName(NAME_None))
	{
		FLockedWritePhysicsObjectExternalInterface Interface = FPhysicsObjectExternalInterface::LockWrite({ &Object, 1 });
		Interface->SetLinearEtherDrag({ &Object, 1 }, InDamping);
	}
}

float UPrimitiveComponent::GetLinearDamping() const
{

	FBodyInstance* BI = GetBodyInstance();
	if (BI)
	{
		return BI->LinearDamping;
	}
	
	return 0.f;
}

void UPrimitiveComponent::SetAngularDamping(float InDamping)
{
	FBodyInstance* BI = GetBodyInstance();
	if (BI)
	{
		BI->AngularDamping = InDamping;
		BI->UpdateDampingProperties();
	}

	else if (Chaos::FPhysicsObject* Object = GetPhysicsObjectByName(NAME_None))
	{
		FLockedWritePhysicsObjectExternalInterface Interface = FPhysicsObjectExternalInterface::LockWrite({ &Object, 1 });
		Interface->SetAngularEtherDrag({ &Object, 1 }, InDamping);
	}
}

float UPrimitiveComponent::GetAngularDamping() const
{
	FBodyInstance* BI = GetBodyInstance();
	if (BI)
	{
		return BI->AngularDamping;
	}
	
	return 0.f;
}

void UPrimitiveComponent::SetMassScale(FName BoneName, float InMassScale)
{
	if (FBodyInstance* BI = GetBodyInstance(BoneName))
	{
		WarnInvalidPhysicsOperations(LOCTEXT("SetMassScale", "SetMassScale"), nullptr, BoneName);
		BI->SetMassScale(InMassScale);
	}
}

float UPrimitiveComponent::GetMassScale(FName BoneName /*= NAME_None*/) const
{
	if (FBodyInstance* BI = GetBodyInstance(BoneName))
	{
		return BI->MassScale;
	}

	return 0.0f;
}

void UPrimitiveComponent::SetAllMassScale(float InMassScale)
{
	SetMassScale(NAME_None, InMassScale);
}

void UPrimitiveComponent::SetMassOverrideInKg(FName BoneName, float MassInKg, bool bOverrideMass)
{
	if (FBodyInstance* BI = GetBodyInstance(BoneName))
	{
		WarnInvalidPhysicsOperations(LOCTEXT("SetCenterOfMass", "SetCenterOfMass"), nullptr, BoneName);
		BI->SetMassOverride(MassInKg, bOverrideMass);
		BI->UpdateMassProperties();
	}
}

float UPrimitiveComponent::GetMass() const
{
	if (FBodyInstance* BI = GetBodyInstance())
	{
		WarnInvalidPhysicsOperations(LOCTEXT("GetMass", "GetMass"), BI, NAME_None);
		return BI->GetBodyMass();
	}
	else
	{
		TArray<Chaos::FPhysicsObject*> PhysicsObjects = GetAllPhysicsObjects();
		if (!PhysicsObjects.IsEmpty())
		{
			FLockedWritePhysicsObjectExternalInterface Interface = FPhysicsObjectExternalInterface::LockWrite(PhysicsObjects);

			// Filter out inactive particles (E.g., inactive children in a geometry collection)
			PhysicsObjects = PhysicsObjects.FilterByPredicate(
				[&Interface](Chaos::FPhysicsObject* Object) 
				{
					return !Interface->AreAllDisabled({ &Object, 1 });
				});

			return Interface->GetMass(PhysicsObjects);
		}
	}

	return 0.0f;
}

FVector UPrimitiveComponent::GetInertiaTensor(FName BoneName /* = NAME_None */) const 
{
	if(FBodyInstance* BI = GetBodyInstance(BoneName))
	{
		return BI->GetBodyInertiaTensor();
	}

	return FVector::ZeroVector;
}

FVector UPrimitiveComponent::ScaleByMomentOfInertia(FVector InputVector, FName BoneName /* = NAME_None */) const
{
	const FVector LocalInertiaTensor = GetInertiaTensor(BoneName);
	const FVector InputVectorLocal = GetComponentTransform().InverseTransformVectorNoScale(InputVector);
	const FVector LocalScaled = InputVectorLocal * LocalInertiaTensor;
	const FVector WorldScaled = GetComponentTransform().TransformVectorNoScale(LocalScaled);
	return WorldScaled;
}

float UPrimitiveComponent::CalculateMass(FName)
{
	if (BodyInstance.bOverrideMass)
	{
		return BodyInstance.GetMassOverride();
	}

	if (BodyInstance.BodySetup.IsValid())
	{
		return BodyInstance.GetBodySetup()->CalculateMass(this);
	}
	else if (UBodySetup * BodySetup = GetBodySetup())
	{
		return BodySetup->CalculateMass(this);
	}
	return 0.0f;
}

float UPrimitiveComponent::GetMaxDepenetrationVelocity(FName BoneName)
{
	FBodyInstance* BI = GetBodyInstance(BoneName);
	if (BI)
	{
		return BI->GetMaxDepenetrationVelocity();
	}
	// @todo: add PhysicsObject support

	// Negative means the config default value will be used
	return -1.0f;
}

void UPrimitiveComponent::SetMaxDepenetrationVelocity(FName BoneName, float InMaxDepenetrationVelocity)
{
	FBodyInstance* BI = GetBodyInstance(BoneName);
	if (BI)
	{
		BI->SetMaxDepenetrationVelocity(InMaxDepenetrationVelocity);
	}
	// @todo: add PhysicsObject support
}

void UPrimitiveComponent::SetUseCCD(bool bInUseCCD, FName BoneName)
{
	FBodyInstance* BI = GetBodyInstance(BoneName);
	if (BI)
	{
		BI->SetUseCCD(bInUseCCD);
	}
}

void UPrimitiveComponent::SetAllUseCCD(bool bInUseCCD)
{
	SetUseCCD(bInUseCCD, NAME_None);
}


void UPrimitiveComponent::SetUseMACD(bool bInUseMACD, FName BoneName)
{
	FBodyInstance* BI = GetBodyInstance(BoneName);
	if (BI)
	{
		BI->SetUseMACD(bInUseMACD);
	}
}

void UPrimitiveComponent::SetAllUseMACD(bool bInUseMACD)
{
	SetUseMACD(bInUseMACD, NAME_None);
}

void UPrimitiveComponent::SetAllowPartialIslandSleep(bool bInAllowPartialIslandSleep /* = true */, FName BoneName /* = NAME_None */)
{
	FBodyInstance* BI = GetBodyInstance(BoneName);
	if (BI)
	{
		BI->SetAllowPartialIslandSleep(bInAllowPartialIslandSleep);
	}
}

void UPrimitiveComponent::SetAllAllowPartialIslandSleep(bool bInAllowPartialIslandSleep /* = true */)
{
	SetAllowPartialIslandSleep(bInAllowPartialIslandSleep, NAME_None);
}

void UPrimitiveComponent::PutRigidBodyToSleep(FName BoneName)
{
	FBodyInstance* BI = GetBodyInstance(BoneName);
	if(BI)
	{
		BI->PutInstanceToSleep();
	}
	else if (BoneName == NAME_None)
	{
		TArray<Chaos::FPhysicsObject*> PhysicsObjects = GetAllPhysicsObjects();
		FLockedWritePhysicsObjectExternalInterface Interface = FPhysicsObjectExternalInterface::LockWrite(PhysicsObjects);

		PhysicsObjects = PhysicsObjects.FilterByPredicate(
			[&Interface](Chaos::FPhysicsObject* Object) {
				return !Interface->AreAllDisabled({ &Object, 1 });
			}
		);
		Interface->PutToSleep(PhysicsObjects);
	}
	else if (Chaos::FPhysicsObject* Object = GetPhysicsObjectByName(BoneName))
	{
		FLockedWritePhysicsObjectExternalInterface Interface = FPhysicsObjectExternalInterface::LockWrite({ &Object, 1 });
		Interface->PutToSleep({ &Object, 1 });
	}
}

void UPrimitiveComponent::PutAllRigidBodiesToSleep()
{
	PutRigidBodyToSleep(NAME_None);
}


bool UPrimitiveComponent::RigidBodyIsAwake(FName BoneName) const
{
	FBodyInstance* BI = GetBodyInstance(BoneName);
	if(BI)
	{
		return BI->IsInstanceAwake();
	}

	return false;
}

bool UPrimitiveComponent::IsAnyRigidBodyAwake()
{
	return RigidBodyIsAwake(NAME_None);
}


void UPrimitiveComponent::SetNotifyRigidBodyCollision(bool bNewNotifyRigidBodyCollision)
{
	BodyInstance.SetInstanceNotifyRBCollision(bNewNotifyRigidBodyCollision);
	OnComponentCollisionSettingsChanged();
}



void UPrimitiveComponent::SetPhysMaterialOverride(UPhysicalMaterial* NewPhysMaterial)
{
	BodyInstance.SetPhysMaterialOverride(NewPhysMaterial);
}

FTransform UPrimitiveComponent::GetComponentTransformFromBodyInstance(FBodyInstance* UseBI)
{
	return UseBI->GetUnrealWorldTransform();
}

void UPrimitiveComponent::SyncComponentToRBPhysics()
{
	if(!IsRegistered())
	{
		UE_LOGF(LogPhysics, Log, "SyncComponentToRBPhysics : Component not registered (%ls)", *GetPathName());
		return;
	}

	 // BodyInstance we are going to sync the component to
	FBodyInstance* UseBI = GetBodyInstance();
	if(UseBI == NULL || !UseBI->IsValidBodyInstance())
	{
		UE_LOGF(LogPhysics, Log, "SyncComponentToRBPhysics : Missing or invalid BodyInstance (%ls)", *GetPathName());
		return;
	}

	AActor* Owner = GetOwner();
	if(Owner != NULL)
	{
		if (!IsValid(Owner) || !Owner->CheckStillInWorld())
		{
			return;
		}
	}

	if (!IsValidChecked(this) || !IsSimulatingPhysics() || !RigidBodyIsAwake())
	{
		return;
	}

	// See if the transform is actually different, and if so, move the component to match physics
	const FTransform NewTransform = GetComponentTransformFromBodyInstance(UseBI);	
	if(!NewTransform.EqualsNoScale(GetComponentTransform()))
	{
		const FVector MoveBy = NewTransform.GetLocation() - GetComponentTransform().GetLocation();
		const FRotator NewRotation = NewTransform.Rotator();

		//@warning: do not reference BodyInstance again after calling MoveComponent() - events from the move could have made it unusable (destroying the actor, SetPhysics(), etc)
		MoveComponent(MoveBy, NewRotation, false, NULL, MOVECOMP_SkipPhysicsMove);
	}
}


UPrimitiveComponent * GetRootWelded(const UPrimitiveComponent* PrimComponent, FName ParentSocketName = NAME_None, FName* OutSocketName = NULL, bool bAboutToWeld = false)
{
	UPrimitiveComponent * Result = NULL;
	UPrimitiveComponent * RootComponent = Cast<UPrimitiveComponent>(PrimComponent->GetAttachParent());	//we must find the root component along hierarchy that has bWelded set to true

	//check that body itself is welded
	if (FBodyInstance* BI = PrimComponent->GetBodyInstance(ParentSocketName, false))
	{
		if (bAboutToWeld == false && BI->WeldParent == nullptr && BI->bAutoWeld == false)	//we're not welded and we aren't trying to become welded
		{
			return NULL;
		}
	}

	FName PrevSocketName = ParentSocketName;
	FName SocketName = NAME_None; //because of skeletal mesh it's important that we check along the bones that we attached
	FBodyInstance* RootBI = NULL;
	for (; RootComponent; RootComponent = Cast<UPrimitiveComponent>(RootComponent->GetAttachParent()))
	{
		Result = RootComponent;
		SocketName = PrevSocketName;
		PrevSocketName = RootComponent->GetAttachSocketName();

		RootBI = RootComponent->GetBodyInstance(SocketName, false);
		if (RootBI && (!RootBI->IsValidBodyInstance() || RootBI->WeldParent != nullptr))
		{
			continue;
		}

		break;
	}

	if (OutSocketName)
	{
		*OutSocketName = SocketName;
	}

	return Result;
}
 
void UPrimitiveComponent::GetWeldedBodies(TArray<FBodyInstance*> & OutWeldedBodies, TArray<FName> & OutLabels, bool bIncludingAutoWeld)
{
	OutWeldedBodies.Add(&BodyInstance);
	OutLabels.Add(NAME_None);

	for (USceneComponent * Child : GetAttachChildren())
	{
		if (UPrimitiveComponent * PrimChild = Cast<UPrimitiveComponent>(Child))
		{
			if (FBodyInstance* BI = PrimChild->GetBodyInstance(NAME_None, false))
			{
				if (BI->WeldParent != nullptr || (bIncludingAutoWeld && BI->bAutoWeld))
				{
					PrimChild->GetWeldedBodies(OutWeldedBodies, OutLabels, bIncludingAutoWeld);
				}
			}
		}
	}
}

void GetRootBodyAndBoneName(const UPrimitiveComponent* RootComponent, const FName& SocketName, FBodyInstance*& OutRootBI, FName& OutBoneName)
{
	FBodyInstance* RootBI = RootComponent->GetBodyInstance(SocketName, false);
	FName BoneName = SocketName;

	//If initial lookup fails, check if we need to lookup the bone from the socket instead
	if (!RootBI && RootComponent->IsA<USkeletalMeshComponent>())
	{
		const USkeletalMeshComponent* RootSkeletal = Cast<USkeletalMeshComponent>(RootComponent);
		if (const USkeletalMeshSocket* Socket = RootSkeletal->GetSocketByName(SocketName))
		{
			BoneName = Socket->BoneName;
			RootBI = RootComponent->GetBodyInstance(BoneName, false);
		}
	}

	OutRootBI = RootBI;
	OutBoneName = BoneName;
}

bool UPrimitiveComponent::WeldToImplementation(USceneComponent * InParent, FName ParentSocketName /* = Name_None */, bool bWeldSimulatedChild /* = true */, bool bWeldToKinematicParent /* = false */)
{
	SCOPE_CYCLE_COUNTER(STAT_WeldPhysics);

	//WeldToInternal assumes attachment is already done
	if (GetAttachParent() != InParent || GetAttachSocketName() != ParentSocketName)
	{
		return false;
	}

	//Check that we can actually our own socket name
	FBodyInstance* BI = GetBodyInstance(NAME_None, false);
	if (BI == NULL)
	{
		return false;
	}

	if (BI->ShouldInstanceSimulatingPhysics() && bWeldSimulatedChild == false)
	{
		return false;
	}

	//Make sure that objects marked as non-simulating do not start simulating due to welding.
	ECollisionEnabled::Type CollisionType = BI->GetCollisionEnabled();
	if (CollisionType == ECollisionEnabled::QueryOnly || CollisionType == ECollisionEnabled::NoCollision)
	{
		return false;
	}

	UnWeldFromParent();	//make sure to unweld from wherever we currently are

	FName SocketName;
	UPrimitiveComponent * RootComponent = GetRootWelded(this, ParentSocketName, &SocketName, true);

	if (RootComponent)
	{
		FBodyInstance* RootBI = nullptr;
		FName BoneName;
		GetRootBodyAndBoneName(RootComponent, SocketName, RootBI, BoneName);

		if (RootBI)
		{
			if (BI->WeldParent == RootBI)	//already welded so stop
			{
				return true;
			}

			//There are multiple cases to handle:
			//Root is kinematic, simulated
			//Child is kinematic, simulated
			//Child always inherits from root

			//if root is kinematic simply set child to be kinematic and we're done
			if ((RootComponent->IsSimulatingPhysics(BoneName) == false) && (bWeldToKinematicParent == false))
			{
				void* OriginalWeldParent = nullptr;

				// Because this uses atomics we need to run it non-transactionally.
				UE_AUTORTFM_OPEN { OriginalWeldParent = FPlatformAtomics::InterlockedExchangePtr((void**)&BI->WeldParent, nullptr); };

				// But remember that on abort we need to fudge the pointer back into the place we took it from.
				AutoRTFM::OnAbort([BI, OriginalWeldParent]
					{
						void* Old = FPlatformAtomics::InterlockedExchangePtr((void**)&BI->WeldParent, OriginalWeldParent);
						ensure(nullptr == Old);
					});

				SetSimulatePhysics(false);
				return false;	//return false because we need to continue with regular body initialization
			}

			//root is simulated so we actually weld the body
			FTransform RelativeTM = RootComponent == GetAttachParent() ? GetRelativeTransform() : GetComponentToWorld().GetRelativeTransform(RootComponent->GetComponentToWorld());	//if direct parent we already have relative. Otherwise compute it
			RootBI->Weld(BI, GetComponentToWorld());

			return true;
		}
	}

	return false;
}

void UPrimitiveComponent::WeldTo(USceneComponent* InParent, FName InSocketName /* = NAME_None */, bool bWeldToKinematicParent /* = false */)
{
	//automatically attach if needed
	if (GetAttachParent() != InParent || GetAttachSocketName() != InSocketName)
	{
		AttachToComponent(InParent, FAttachmentTransformRules::KeepWorldTransform, InSocketName);
	}

	const bool bWeldSimulatedChild = true;
	WeldToImplementation(InParent, InSocketName, bWeldSimulatedChild, bWeldToKinematicParent);
}

void UPrimitiveComponent::UnWeldFromParent()
{
	FBodyInstance* NewRootBI = GetBodyInstance(NAME_None, false);
	UWorld* CurrentWorld = GetWorld();
	if (NewRootBI == NULL || NewRootBI->WeldParent == nullptr || CurrentWorld == nullptr || CurrentWorld->GetPhysicsScene() == nullptr || !IsValidChecked(this) || IsUnreachable())
	{
		return;
	}

	// If we're purging (shutting down everything to kill the runtime) don't proceed
	// to make new physics bodies and weld them, as they'll never be used.
	if(GExitPurge)
	{
		return;
	}

	SCOPE_CYCLE_COUNTER(STAT_UnweldPhysics);

	FName SocketName;
	UPrimitiveComponent * RootComponent = GetRootWelded(this, GetAttachSocketName(), &SocketName);

	if (RootComponent)
	{
		FBodyInstance* RootBI = nullptr;
		FName BoneName;
		GetRootBodyAndBoneName(RootComponent, SocketName, RootBI, BoneName);

		if (RootBI != nullptr)
		{
			bool bRootIsBeingDeleted = !IsValidChecked(RootComponent) || RootComponent->IsUnreachable();
			const FBodyInstance* PrevWeldParent = NewRootBI->WeldParent;
			RootBI->UnWeld(NewRootBI);

			{
				void* OriginalWeldParent = nullptr;

				// Because this uses atomics we need to run it non-transactionally.
				UE_AUTORTFM_OPEN { OriginalWeldParent = FPlatformAtomics::InterlockedExchangePtr((void**)&NewRootBI->WeldParent, nullptr); };

				// But remember that on abort we need to fudge the pointer back into the place we took it from.
				AutoRTFM::OnAbort([NewRootBI, OriginalWeldParent]
					{
						void* Old = FPlatformAtomics::InterlockedExchangePtr((void**)&NewRootBI->WeldParent, OriginalWeldParent);
						ensure(nullptr == Old);
					});
			}

			bool bHasBodySetup = GetBodySetup() != nullptr;

			//if BodyInstance hasn't already been created we need to initialize it
			if (!bRootIsBeingDeleted && bHasBodySetup && NewRootBI->IsValidBodyInstance() == false)
			{
				bool bPrevAutoWeld = NewRootBI->bAutoWeld;
				NewRootBI->bAutoWeld = false;
				NewRootBI->InitBody(GetBodySetup(), GetComponentToWorld(), this, CurrentWorld->GetPhysicsScene());
				NewRootBI->bAutoWeld = bPrevAutoWeld;
			}

			if(PrevWeldParent == nullptr)	//our parent is kinematic so no need to do any unwelding/rewelding of children
			{
				return;
			}

			//now weld its children to it
			TArray<FBodyInstance*> ChildrenBodies;
			TArray<FName> ChildrenLabels;
			GetWeldedBodies(ChildrenBodies, ChildrenLabels);

			for (int32 ChildIdx = 0; ChildIdx < ChildrenBodies.Num(); ++ChildIdx)
			{
				FBodyInstance* ChildBI = ChildrenBodies[ChildIdx];
				checkSlow(ChildBI);
				if (ChildBI != NewRootBI)
				{
					if (!bRootIsBeingDeleted)
					{
						RootBI->UnWeld(ChildBI);
					}

					//At this point, NewRootBI must be kinematic because it's being unwelded.

					void* OriginalWeldParent = nullptr;

					// Because this uses atomics we need to run it non-transactionally.
					//null because we are currently kinematic
					UE_AUTORTFM_OPEN { OriginalWeldParent = FPlatformAtomics::InterlockedExchangePtr((void**)&ChildBI->WeldParent, nullptr); };

					// But remember that on abort we need to fudge the pointer back into the place we took it from.
					AutoRTFM::OnAbort([ChildBI, OriginalWeldParent]
						{
							void* Old = FPlatformAtomics::InterlockedExchangePtr((void**)&ChildBI->WeldParent, OriginalWeldParent);
							ensure(nullptr == Old);
						});
				}
			}

			//If the new root body is simulating, we need to apply the weld on the children
			if(!bRootIsBeingDeleted && NewRootBI->IsInstanceSimulatingPhysics())
			{
				NewRootBI->ApplyWeldOnChildren();
			}
			
		}
	}
}

void UPrimitiveComponent::UnWeldChildren()
{
	for (TObjectPtr<USceneComponent> ChildComponent : GetAttachChildren())
	{
		if (ChildComponent.GetResidence() == EResidence::Local)
		{
			if (UPrimitiveComponent* PrimComp = Cast<UPrimitiveComponent>(ChildComponent))
			{
				PrimComp->UnWeldFromParent();
			}
		}
	}
}

FBodyInstance* UPrimitiveComponent::GetBodyInstance(FName BoneName, bool bGetWelded, int32 Index) const
{
	return const_cast<FBodyInstance*>((bGetWelded && BodyInstance.WeldParent) ? BodyInstance.WeldParent : &BodyInstance);
}

FBodyInstanceAsyncPhysicsTickHandle UPrimitiveComponent::GetBodyInstanceAsyncPhysicsTickHandle(FName BoneName, bool bGetWelded, int32 Index) const
{
	if(FBodyInstance* BI = GetBodyInstance(BoneName, bGetWelded, Index))
	{
		return BI->GetBodyInstanceAsyncPhysicsTickHandle();
	}
	else
	{
		return FBodyInstanceAsyncPhysicsTickHandle();
	}
	
}

bool UPrimitiveComponent::GetSquaredDistanceToCollision(const FVector& Point, float& OutSquaredDistance, FVector& OutClosestPointOnCollision) const
{
	OutClosestPointOnCollision = Point;

	FBodyInstance* BodyInst = GetBodyInstance();
	if (BodyInst != nullptr)
	{
		return BodyInst->GetSquaredDistanceToBody(Point, OutSquaredDistance, OutClosestPointOnCollision);
	}

	return false;
}

float UPrimitiveComponent::GetClosestPointOnCollision(const FVector& Point, FVector& OutPointOnBody, FName BoneName) const
{
	OutPointOnBody = Point;

	if (FBodyInstance* BodyInst = GetBodyInstance(BoneName, /*bGetWelded=*/ false))
	{
		return BodyInst->GetDistanceToBody(Point, OutPointOnBody);
	}

	return -1.f;
}

bool UPrimitiveComponent::IsSimulatingPhysics(FName BoneName) const
{
	if(FBodyInstance* BodyInst = GetBodyInstance(BoneName))
	{
		return BodyInst->IsInstanceSimulatingPhysics();
	}
	else
	{
		TArray<Chaos::FPhysicsObjectHandle> PhysicsObjects;
		if (BoneName != NAME_None)
		{
			PhysicsObjects.Add(GetPhysicsObjectByName(BoneName));
		}
		else
		{
			PhysicsObjects = GetAllPhysicsObjects();
		}

		if (PhysicsObjects.IsEmpty())
		{
			return false;
		}

		FLockedReadPhysicsObjectExternalInterface Interface = FPhysicsObjectExternalInterface::LockRead(PhysicsObjects);

		PhysicsObjects.RemoveAllSwap(
			[&Interface](Chaos::FPhysicsObjectHandle Object)
			{
				return Interface->AreAllDisabled({ &Object, 1 });
			},
			EAllowShrinking::No);

		return Interface->AreAllDynamicOrSleeping(PhysicsObjects);
	}
}

FVector UPrimitiveComponent::GetComponentVelocity() const
{
	if (IsSimulatingPhysics())
	{
		FBodyInstance* BodyInst = GetBodyInstance();
		if(BodyInst != NULL)
		{
			return BodyInst->GetUnrealWorldVelocity();
		}
	}

	return Super::GetComponentVelocity();
}

void UPrimitiveComponent::SetCollisionObjectType(ECollisionChannel Channel)
{
	BodyInstance.SetObjectType(Channel);
}

void UPrimitiveComponent::SetCollisionResponseToChannel(ECollisionChannel Channel, ECollisionResponse NewResponse)
{
	if (BodyInstance.SetResponseToChannel(Channel, NewResponse))
	{ 
		OnComponentCollisionSettingsChanged();
	}
}

void UPrimitiveComponent::SetCollisionResponseToAllChannels(enum ECollisionResponse NewResponse)
{
	if (BodyInstance.SetResponseToAllChannels(NewResponse))
	{
		OnComponentCollisionSettingsChanged();
	}
}

void UPrimitiveComponent::SetCollisionResponseToChannels(const FCollisionResponseContainer& NewReponses)
{	
	if (BodyInstance.SetResponseToChannels(NewReponses))
	{ 
		OnComponentCollisionSettingsChanged();
	}
}

void UPrimitiveComponent::SetCollisionEnabled(ECollisionEnabled::Type NewType)
{
	ECollisionEnabled::Type CurrentType = BodyInstance.GetCollisionEnabled();

	if (CurrentType != NewType)
	{
		BodyInstance.SetCollisionEnabled(NewType);

		EnsurePhysicsStateCreated();
		OnComponentCollisionSettingsChanged();

		if(IsRegistered() && BodyInstance.bSimulatePhysics && !IsWelded())
		{
			
			BodyInstance.ApplyWeldOnChildren();
		}

	}
}

// @todo : implement skeletalmeshcomponent version
void UPrimitiveComponent::SetCollisionProfileName(FName InCollisionProfileName, bool bUpdateOverlaps)
{
	SCOPE_CYCLE_COUNTER(STAT_PrimComp_SetCollisionProfileName);

	FUObjectThreadContext& ThreadContext = FUObjectThreadContext::Get();
	if (ThreadContext.ConstructedObject == this)
	{
		// If we are in our constructor, defer setup until PostInitProperties as derived classes
		// may call SetCollisionProfileName more than once.
		BodyInstance.SetCollisionProfileNameDeferred(InCollisionProfileName);
	}
	else if (InCollisionProfileName != BodyInstance.GetCollisionProfileName())
	{
		ECollisionEnabled::Type OldCollisionEnabled = BodyInstance.GetCollisionEnabled();
		BodyInstance.SetCollisionProfileName(InCollisionProfileName);

		ECollisionEnabled::Type NewCollisionEnabled = BodyInstance.GetCollisionEnabled();

		if (OldCollisionEnabled != NewCollisionEnabled)
		{
			EnsurePhysicsStateCreated();
		}
		OnComponentCollisionSettingsChanged(bUpdateOverlaps);
	}
}

FName UPrimitiveComponent::GetCollisionProfileName() const
{
	return BodyInstance.GetCollisionProfileName();
}

void UPrimitiveComponent::OnActorEnableCollisionChanged()
{
	BodyInstance.UpdatePhysicsFilterData();
	OnComponentCollisionSettingsChanged(false);
}

void UPrimitiveComponent::OnComponentCollisionSettingsChanged(bool bUpdateOverlaps)
{
	if (IsRegistered() && !IsTemplate())			// not for CDOs
	{
		// changing collision settings could affect touching status, need to update
		if (IsQueryCollisionEnabled())	//if we have query collision we may now care about overlaps so clear cache
		{
			ClearSkipUpdateOverlaps();
		}

		if (bUpdateOverlaps)
		{
			UpdateOverlaps();
		}

		// update navigation data if needed
		const bool bNewNavRelevant = IsNavigationRelevant();
		if (bNavigationRelevant != bNewNavRelevant)
		{
			bNavigationRelevant = bNewNavRelevant;
			FNavigationSystem::UpdateComponentData(*this);
		}

		OnComponentCollisionSettingsChangedEvent.Broadcast(this);
	}
}

bool UPrimitiveComponent::K2_LineTraceComponent(FVector TraceStart, FVector TraceEnd, bool bTraceComplex, bool bShowTrace, bool bPersistentShowTrace, FVector& HitLocation, FVector& HitNormal, FName& BoneName, FHitResult& OutHit)
{
	FCollisionQueryParams LineParams(SCENE_QUERY_STAT(KismetTraceComponent), bTraceComplex);
	const bool bDidHit = LineTraceComponent(OutHit, TraceStart, TraceEnd, LineParams);

	if( bDidHit )
	{
		// Fill in the results if we hit
		HitLocation = OutHit.Location;
		HitNormal = OutHit.Normal;
		BoneName = OutHit.BoneName;
	}
	else
	{
		// Blank these out to avoid confusion!
		HitLocation = FVector::ZeroVector;
		HitNormal = FVector::ZeroVector;
		BoneName = NAME_None;
	}

	if( bShowTrace )
	{
		DrawDebugLine(GetWorld(), TraceStart, bDidHit ? HitLocation : TraceEnd, FColor(255, 128, 0), bPersistentShowTrace, -1.0f, 0, 2.0f);
		if(bDidHit)
		{
			DrawDebugLine(GetWorld(), HitLocation, TraceEnd, FColor(0, 128, 255), bPersistentShowTrace, -1.0f, 0, 2.0f);
		}
	}

	return bDidHit;
}

bool UPrimitiveComponent::K2_SphereTraceComponent(FVector TraceStart, FVector TraceEnd, float SphereRadius, bool bTraceComplex, bool bShowTrace, bool bPersistentShowTrace, FVector& HitLocation, FVector& HitNormal, FName& BoneName, FHitResult& OutHit)
{
	FCollisionShape SphereShape;
	SphereShape.SetSphere(SphereRadius);
	bool bDidHit = SweepComponent(OutHit, TraceStart, TraceEnd, FQuat::Identity, SphereShape, bTraceComplex);

	if(bDidHit)
	{
		// Fill in the results if we hit
		HitLocation = OutHit.Location;
		HitNormal = OutHit.Normal;
		BoneName = OutHit.BoneName;
	}
	else
	{
		// Blank these out to avoid confusion!
		HitLocation = FVector::ZeroVector;
		HitNormal = FVector::ZeroVector;
		BoneName = NAME_None;
	}

	if(bShowTrace)
	{
		DrawDebugLine(GetWorld(), TraceStart, bDidHit ? HitLocation : TraceEnd, FColor(255, 128, 0), bPersistentShowTrace, -1.0f, 0, 2.0f);
		if(bDidHit)
		{
			DrawDebugLine(GetWorld(), HitLocation, TraceEnd, FColor(0, 128, 255), bPersistentShowTrace, -1.0f, 0, 2.0f);
			DrawDebugSphere(GetWorld(), HitLocation, SphereRadius, 16, FColor(255, 0, 0), bPersistentShowTrace, -1.0f, 0, 0.25f);
		}
	}

	return bDidHit;
}

bool UPrimitiveComponent::K2_BoxOverlapComponent(FVector InBoxCentre, const FBox InBox, bool bTraceComplex, bool bShowTrace, bool bPersistentShowTrace, FVector& HitLocation, FVector& HitNormal, FName& BoneName, FHitResult& OutHit)
{
	FCollisionShape QueryBox = FCollisionShape::MakeBox(InBox.GetExtent());

	TArray<FOverlapResult> OverlapResult;
	bool bHit = OverlapComponentWithResult(InBoxCentre, FQuat::Identity, QueryBox, OverlapResult);
	if (bHit && !OverlapResult.IsEmpty())
	{
		OutHit = ConvertOverlapToHitResult(OverlapResult[0]);
	}

	if(bShowTrace)
	{
		FColor BoxColor = bHit ? FColor::Red : FColor::Green;

		DrawDebugBox(GetWorld(), InBoxCentre, QueryBox.GetExtent(), FQuat::Identity, BoxColor, bPersistentShowTrace, -1.0f, 0, 0.4f);
	}

	return bHit;
}

bool UPrimitiveComponent::K2_SphereOverlapComponent(FVector InSphereCentre, float InSphereRadius, bool bTraceComplex, bool bShowTrace, bool bPersistentShowTrace, FVector& HitLocation, FVector& HitNormal, FName& BoneName, FHitResult& OutHit)
{
	FCollisionShape QuerySphere = FCollisionShape::MakeSphere(InSphereRadius);

	TArray<FOverlapResult> OverlapResult;
	bool bHit = OverlapComponentWithResult(InSphereCentre, FQuat::Identity, QuerySphere, OverlapResult);
	if (bHit && !OverlapResult.IsEmpty())
	{
		OutHit = ConvertOverlapToHitResult(OverlapResult[0]);
	}

	if(bShowTrace)
	{
		FColor SphereColor = bHit ? FColor::Red : FColor::Green;

		DrawDebugSphere(GetWorld(), InSphereCentre, QuerySphere.GetSphereRadius(), 16, SphereColor, bPersistentShowTrace, -1.0f, 0, 0.4f);
	}

	return bHit;
}

ECollisionEnabled::Type UPrimitiveComponent::GetCollisionEnabled() const
{
	AActor* Owner = GetOwner();
	if (Owner && !Owner->GetActorEnableCollision())
	{
		return ECollisionEnabled::NoCollision;
	}

	return BodyInstance.GetCollisionEnabled(false);
}

ECollisionResponse UPrimitiveComponent::GetCollisionResponseToChannel(ECollisionChannel Channel) const
{
	return BodyInstance.GetResponseToChannel(Channel);
}

const FCollisionResponseContainer& UPrimitiveComponent::GetCollisionResponseToChannels() const
{
	return BodyInstance.GetResponseToChannels();
}

void UPrimitiveComponent::UpdatePhysicsToRBChannels()
{
	if(BodyInstance.IsValidBodyInstance())
	{
		BodyInstance.UpdatePhysicsFilterData();
	}
}

Chaos::FPhysicsObject* UPrimitiveComponent::GetPhysicsObjectById(Chaos::FPhysicsObjectId Id) const
{
	if (!BodyInstance.IsValidBodyInstance())
	{
		return nullptr;
	}

	return BodyInstance.GetPhysicsActor()->GetPhysicsObject();
}

Chaos::FPhysicsObject* UPrimitiveComponent::GetPhysicsObjectByName(const FName& Name) const
{
	return GetPhysicsObjectById(0);
}

TArray<Chaos::FPhysicsObject*> UPrimitiveComponent::GetAllPhysicsObjects() const
{
	TArray<Chaos::FPhysicsObject*> Bodies = { GetPhysicsObjectById(INDEX_NONE) };
	return Bodies;
}

Chaos::FPhysicsObjectId UPrimitiveComponent::GetIdFromGTParticle(Chaos::FGeometryParticle* Particle) const
{
	return 0;
}

IPhysicsBodyInstanceOwner* UPrimitiveComponent::ResolvePhysicsBodyInstanceOwner(Chaos::FConstPhysicsObjectHandle PhysicsObject)
{
	return this;
}

UObject* UPrimitiveComponent::GetSourceObjectOwner() const
{
	return GetOwner();
}

UPhysicalMaterial* UPrimitiveComponent::GetPhysicsMaterialOverride() const
{
	return BodyInstance.GetPhysMaterialOverride();
}

UMaterialInterface* UPrimitiveComponent::GetPhysicsMaterialBase() const
{
	return IsA<UMeshComponent>() ? GetMaterial(0) : nullptr;
}

bool UPrimitiveComponent::IsPhysicsOwnerMovable() const
{
	return Mobility == EComponentMobility::Movable;
}

bool UPrimitiveComponent::IsPhysicsOwnerSimulatingPhysics() const
{
	bool bBaseIsSimulatingPhysics = false;
	const USceneComponent* AttachParentComponent = this;
	while (!bBaseIsSimulatingPhysics && AttachParentComponent)
	{
		bBaseIsSimulatingPhysics = AttachParentComponent->IsSimulatingPhysics();
		AttachParentComponent = AttachParentComponent->GetAttachParent();
	}
	
	return bBaseIsSimulatingPhysics;
}

FVector UPrimitiveComponent::GetPhysicsOwnerVelocity() const
{
	return GetComponentVelocity();
}

UObject* UPrimitiveComponent::GetPhysicsOwnerAttachmentRoot() const
{
	return GetAttachmentRoot();
}

bool UPrimitiveComponent::IsPhysicsObjectWorldGeometry() const
{
	return IsWorldGeometry();
}

FTransform UPrimitiveComponent::GetPhysicsOwnerSocketTransform(FName InSocketName) const
{
	return GetSocketTransform(InSocketName);
}

bool UPrimitiveComponent::DoesSocketExistOnPhysicsOwner(FName InSocketName) const
{
	return DoesSocketExist(InSocketName);
}

#undef LOCTEXT_NAMESPACE
