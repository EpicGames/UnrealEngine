// Copyright Epic Games, Inc. All Rights Reserved.

#include "PrimitiveDrawingUtils.h"
#include "Math/RandomStream.h"
#include "MaterialShared.h"
#include "Materials/Material.h"
#include "Materials/MaterialRenderProxy.h"
#include "CanvasItem.h"
#include "CanvasTypes.h"
#include "Rendering/SkeletalMeshLODRenderData.h"
#include "SceneView.h"
#include "UnrealEngine.h"
#include "DynamicMeshBuilder.h"
#include "PrimitiveSceneProxy.h"
#include "Engine/LightMapTexture2D.h"
#include "SceneInterface.h"
#include "UnrealClient.h"
#include "PrimitiveDrawingUtils.h"

#include "Materials/MaterialInstance.h"
#include "Materials/MaterialInstanceDynamic.h"

void DrawPlane10x10(class FPrimitiveDrawInterface* PDI,const FMatrix& ObjectToWorld, float Radii, FVector2D UVMin, FVector2D UVMax, const FMaterialRenderProxy* MaterialRenderProxy, uint8 DepthPriorityGroup)
{
	// -> TileCount * TileCount * 2 triangles
	const uint32 TileCount = 10;

	FDynamicMeshBuilder MeshBuilder(PDI->View->GetFeatureLevel());

	// todo: reserve or better cache the mesh

	float Step = 2.0f / TileCount;

	for(int32 y = 0; y < TileCount; ++y)
	{
		// implemented this way to avoid cracks, could be optimized
		float y0 = y * Step - 1.0f;
		float y1 = (y + 1) * Step - 1.0f;

		float V0 = FMath::Lerp(UVMin.Y, UVMax.Y, y0 * 0.5f + 0.5f);
		float V1 = FMath::Lerp(UVMin.Y, UVMax.Y, y1 * 0.5f + 0.5f);

		for(int32 x = 0; x < TileCount; ++x)
		{
			// implemented this way to avoid cracks, could be optimized
			float x0 = x * Step - 1.0f;
			float x1 = (x + 1) * Step - 1.0f;

			float U0 = FMath::Lerp(UVMin.X, UVMax.X, x0 * 0.5f + 0.5f);
			float U1 = FMath::Lerp(UVMin.X, UVMax.X, x1 * 0.5f + 0.5f);

			// Calculate verts for a face pointing down Z
			MeshBuilder.AddVertex(FVector3f(x0, y0, 0), FVector2f(U0, V0), FVector3f(1, 0, 0), FVector3f(0, 1, 0), FVector3f(0, 0, 1), FColor::White);
			MeshBuilder.AddVertex(FVector3f(x0, y1, 0), FVector2f(U0, V1), FVector3f(1, 0, 0), FVector3f(0, 1, 0), FVector3f(0, 0, 1), FColor::White);
			MeshBuilder.AddVertex(FVector3f(x1, y1, 0), FVector2f(U1, V1), FVector3f(1, 0, 0), FVector3f(0, 1, 0), FVector3f(0, 0, 1), FColor::White);
			MeshBuilder.AddVertex(FVector3f(x1, y0, 0), FVector2f(U1, V0), FVector3f(1, 0, 0), FVector3f(0, 1, 0), FVector3f(0, 0, 1), FColor::White);

			int Index = (x + y * TileCount) * 4;
			MeshBuilder.AddTriangle(Index + 0, Index + 1, Index + 2);
			MeshBuilder.AddTriangle(Index + 0, Index + 2, Index + 3);
		}
	}	

	MeshBuilder.Draw(PDI, FScaleMatrix(Radii) * ObjectToWorld, MaterialRenderProxy, DepthPriorityGroup, 0.f);
}

void DrawTriangle(class FPrimitiveDrawInterface* PDI, const FVector& A, const FVector& B, const FVector& C, const FMaterialRenderProxy* MaterialRenderProxy, uint8 DepthPriorityGroup)
{
	FVector2f UVs[4] =
	{
		FVector2f(0,0),
		FVector2f(0,1),
		FVector2f(1,1),
		FVector2f(1,0),
	};

	FDynamicMeshBuilder MeshBuilder(PDI->View->GetFeatureLevel());

	FVector3f Normal = FVector3f(0, 0, 1);
	FVector3f Tangent = FVector3f(1, 0, 0);	

	MeshBuilder.AddVertex(FDynamicMeshVertex((FVector3f)A, Tangent, Normal, UVs[0],FColor::White));

	MeshBuilder.AddVertex(FDynamicMeshVertex((FVector3f)B, Tangent, Normal, UVs[1], FColor::White));
	MeshBuilder.AddVertex(FDynamicMeshVertex((FVector3f)C, Tangent, Normal, UVs[2], FColor::White));

	MeshBuilder.AddTriangle(0, 1, 2);
	MeshBuilder.Draw(PDI, FMatrix::Identity, MaterialRenderProxy, DepthPriorityGroup, false, false);

	PDI->DrawLine(A, B, FColor::Yellow, DepthPriorityGroup, 1.f);
	PDI->DrawLine(A, C, FColor::Yellow, DepthPriorityGroup, 1.f);
	PDI->DrawLine(B, C, FColor::Yellow, DepthPriorityGroup, 1.f);
}

void GetBoxMesh(const FMatrix& BoxToWorld,const FVector& Radii,const FMaterialRenderProxy* MaterialRenderProxy,uint8 DepthPriorityGroup,int32 ViewIndex,FMeshElementCollector& Collector, HHitProxy* HitProxy)
{
	// Calculate verts for a face pointing down Z
	FVector3f Positions[4] =
	{
		FVector3f(-1, -1, +1),
		FVector3f(-1, +1, +1),
		FVector3f(+1, +1, +1),
		FVector3f(+1, -1, +1)
	};
	FVector2f UVs[4] =
	{
		FVector2f(0, 0),
		FVector2f(0, 1),
		FVector2f(1, 1),
		FVector2f(1, 0),
	};

	// Then rotate this face 6 times
	FRotator3f FaceRotations[6];
	FaceRotations[0] = FRotator3f(0,		0,	0);
	FaceRotations[1] = FRotator3f(90.f,		0,	0);
	FaceRotations[2] = FRotator3f(-90.f,	0,	0);
	FaceRotations[3] = FRotator3f(0,		0,	90.f);
	FaceRotations[4] = FRotator3f(0,		0,	-90.f);
	FaceRotations[5] = FRotator3f(180.f,	0,	0);

	FDynamicMeshBuilder MeshBuilder(Collector.GetFeatureLevel());

	for (int32 f = 0; f < 6; f++)
	{
		FMatrix44f FaceTransform = FRotationMatrix44f(FaceRotations[f]) * FScaleMatrix44f(FVector3f(Radii));

		int32 VertexIndices[4];
		for (int32 VertexIndex = 0; VertexIndex < 4; VertexIndex++)
		{
			VertexIndices[VertexIndex] = MeshBuilder.AddVertex(
				FaceTransform.TransformPosition( Positions[VertexIndex] ),
				UVs[VertexIndex],
				FaceTransform.TransformVector(FVector3f(1, 0, 0)),
				FaceTransform.TransformVector(FVector3f(0, 1, 0)),
				FaceTransform.TransformVector(FVector3f(0, 0, 1)),
				FColor::White
				);
		}

		MeshBuilder.AddTriangle(VertexIndices[0], VertexIndices[1], VertexIndices[2]);
		MeshBuilder.AddTriangle(VertexIndices[0], VertexIndices[2], VertexIndices[3]);
	}

	MeshBuilder.GetMesh(BoxToWorld, MaterialRenderProxy, DepthPriorityGroup, false, false, true, ViewIndex, Collector, HitProxy);
}

void DrawBox(FPrimitiveDrawInterface* PDI,const FMatrix& BoxToWorld,const FVector& Radii,const FMaterialRenderProxy* MaterialRenderProxy,uint8 DepthPriorityGroup)
{
	// Calculate verts for a face pointing down Z
	FVector3f Positions[4] =
	{
		FVector3f(-1, -1, +1),
		FVector3f(-1, +1, +1),
		FVector3f(+1, +1, +1),
		FVector3f(+1, -1, +1)
	};
	FVector2f UVs[4] =
	{
		FVector2f(0, 0),
		FVector2f(0, 1),
		FVector2f(1, 1),
		FVector2f(1, 0),
	};

	// Then rotate this face 6 times
	FRotator3f FaceRotations[6];
	FaceRotations[0] = FRotator3f(0,		0,	0);
	FaceRotations[1] = FRotator3f(90.f,		0,	0);
	FaceRotations[2] = FRotator3f(-90.f,	0,  0);
	FaceRotations[3] = FRotator3f(0,		0,	90.f);
	FaceRotations[4] = FRotator3f(0,		0,	-90.f);
	FaceRotations[5] = FRotator3f(180.f,	0,	0);

	FDynamicMeshBuilder MeshBuilder(PDI->View->GetFeatureLevel());

	for(int32 f = 0; f < 6; f++)
	{
		FMatrix44f FaceTransform = FRotationMatrix44f(FaceRotations[f]) * FScaleMatrix44f(FVector3f(Radii));

		int32 VertexIndices[4];
		for (int32 VertexIndex = 0; VertexIndex < 4; VertexIndex++)
		{
			VertexIndices[VertexIndex] = MeshBuilder.AddVertex(
				FaceTransform.TransformPosition( Positions[VertexIndex] ),
				UVs[VertexIndex],
				FaceTransform.TransformVector(FVector3f(1, 0, 0)),
				FaceTransform.TransformVector(FVector3f(0, 1, 0)),
				FaceTransform.TransformVector(FVector3f(0, 0, 1)),
				FColor::White);
		}

		MeshBuilder.AddTriangle(VertexIndices[0], VertexIndices[1], VertexIndices[2]);
		MeshBuilder.AddTriangle(VertexIndices[0], VertexIndices[2], VertexIndices[3]);
	}

	MeshBuilder.Draw(PDI, BoxToWorld, MaterialRenderProxy, DepthPriorityGroup, 0.f);
}

void GetOrientedHalfSphereMesh(const FVector& Center, const FRotator& Orientation, const FVector& Radii, int32 NumSides, int32 NumRings, float StartAngle, float EndAngle, const FMaterialRenderProxy* MaterialRenderProxy, uint8 DepthPriority,
	bool bDisableBackfaceCulling, int32 ViewIndex, FMeshElementCollector& Collector, bool bUseSelectionOutline, HHitProxy* HitProxy)
{
	// Use a mesh builder to draw the sphere.
	FDynamicMeshBuilder MeshBuilder(Collector.GetFeatureLevel());
	{
		// The first/last arc are on top of each other.
		int32 NumVerts = (NumSides + 1) * (NumRings + 1);
		FDynamicMeshVertex* Verts = (FDynamicMeshVertex*)FMemory::Malloc(NumVerts * sizeof(FDynamicMeshVertex));

		// Calculate verts for one arc
		FDynamicMeshVertex* ArcVerts = (FDynamicMeshVertex*)FMemory::Malloc((NumRings + 1) * sizeof(FDynamicMeshVertex));

		for (int32 i = 0; i<NumRings + 1; i++)
		{
			FDynamicMeshVertex* ArcVert = &ArcVerts[i];

			float angle = StartAngle + ((float)i / NumRings) * (EndAngle-StartAngle);

			// Note- unit sphere, so position always has mag of one. We can just use it for normal!			
			ArcVert->Position.X = 0.0f;
			ArcVert->Position.Y = FMath::Sin(angle);
			ArcVert->Position.Z = FMath::Cos(angle);

			ArcVert->SetTangents(
				FVector3f(1, 0, 0),
				FVector3f(0.0f, -ArcVert->Position.Z, ArcVert->Position.Y),
				ArcVert->Position
				);

			ArcVert->TextureCoordinate[0].X = 0.0f;
			ArcVert->TextureCoordinate[0].Y = ((float)i / NumRings);
		}

		// Then rotate this arc NumSides+1 times.
		for (int32 s = 0; s<NumSides + 1; s++)
		{
			FRotator3f ArcRotator(0, 360.f * (float)s / NumSides, 0);
			FRotationMatrix44f ArcRot(ArcRotator);
			float XTexCoord = ((float)s / NumSides);

			for (int32 v = 0; v<NumRings + 1; v++)
			{
				int32 VIx = (NumRings + 1)*s + v;

				Verts[VIx].Position = ArcRot.TransformPosition(ArcVerts[v].Position);

				Verts[VIx].SetTangents(
					ArcRot.TransformVector(ArcVerts[v].TangentX.ToFVector3f()),
					ArcRot.TransformVector(ArcVerts[v].GetTangentY()),
					ArcRot.TransformVector(ArcVerts[v].TangentZ.ToFVector3f())
					);

				Verts[VIx].TextureCoordinate[0].X = XTexCoord;
				Verts[VIx].TextureCoordinate[0].Y = ArcVerts[v].TextureCoordinate[0].Y;
			}
		}

		// Add all of the vertices we generated to the mesh builder.
		for (int32 VertIdx = 0; VertIdx < NumVerts; VertIdx++)
		{
			MeshBuilder.AddVertex(Verts[VertIdx]);
		}

		// Add all of the triangles we generated to the mesh builder.
		for (int32 s = 0; s<NumSides; s++)
		{
			int32 a0start = (s + 0) * (NumRings + 1);
			int32 a1start = (s + 1) * (NumRings + 1);

			for (int32 r = 0; r<NumRings; r++)
			{
				MeshBuilder.AddTriangle(a0start + r + 0, a1start + r + 0, a0start + r + 1);
				MeshBuilder.AddTriangle(a1start + r + 0, a1start + r + 1, a0start + r + 1);
			}
		}

		// Free our local copy of verts and arc verts
		FMemory::Free(Verts);
		FMemory::Free(ArcVerts);
	}
	MeshBuilder.GetMesh(FScaleMatrix(Radii) * FRotationMatrix(Orientation) * FTranslationMatrix(Center), MaterialRenderProxy, DepthPriority, bDisableBackfaceCulling, false, bUseSelectionOutline, ViewIndex, Collector, HitProxy);
}

void GetHalfSphereMesh(const FVector& Center, const FVector& Radii, int32 NumSides, int32 NumRings, float StartAngle, float EndAngle, const FMaterialRenderProxy* MaterialRenderProxy, uint8 DepthPriority, 
					bool bDisableBackfaceCulling, int32 ViewIndex, FMeshElementCollector& Collector,bool bUseSelectionOutline,HHitProxy* HitProxy)
{
	GetOrientedHalfSphereMesh(Center, FRotator::ZeroRotator, Radii, NumSides, NumRings, StartAngle, EndAngle, MaterialRenderProxy, DepthPriority, bDisableBackfaceCulling, ViewIndex, Collector, bUseSelectionOutline, HitProxy);
}

void GetSphereMesh(const FVector& Center, const FVector& Radii, int32 NumSides, int32 NumRings, const FMaterialRenderProxy* MaterialRenderProxy, uint8 DepthPriority,
					bool bDisableBackfaceCulling, int32 ViewIndex, FMeshElementCollector& Collector)
{
	GetSphereMesh(Center, Radii, NumSides, NumRings, MaterialRenderProxy, DepthPriority, bDisableBackfaceCulling, ViewIndex, Collector, true, NULL);
}

extern ENGINE_API void GetSphereMesh(const FVector& Center, const FVector& Radii, int32 NumSides, int32 NumRings, const FMaterialRenderProxy* MaterialRenderProxy, uint8 DepthPriority,
	bool bDisableBackfaceCulling, int32 ViewIndex, FMeshElementCollector& Collector, bool bUseSelectionOutline, HHitProxy* HitProxy)
{
	GetHalfSphereMesh(Center, Radii, NumSides, NumRings, 0, UE_PI, MaterialRenderProxy, DepthPriority, bDisableBackfaceCulling, ViewIndex, Collector, bUseSelectionOutline, HitProxy);
}

void DrawSphere(FPrimitiveDrawInterface* PDI,const FVector& Center,const FRotator& Orientation,const FVector& Radii,int32 NumSides,int32 NumRings,const FMaterialRenderProxy* MaterialRenderProxy,uint8 DepthPriority,bool bDisableBackfaceCulling)
{
	// Use a mesh builder to draw the sphere.
	FDynamicMeshBuilder MeshBuilder(PDI->View->GetFeatureLevel());
	{
		// The first/last arc are on top of each other.
		int32 NumVerts = (NumSides+1) * (NumRings+1);
		FDynamicMeshVertex* Verts = (FDynamicMeshVertex*)FMemory::Malloc( NumVerts * sizeof(FDynamicMeshVertex) );

		// Calculate verts for one arc
		FDynamicMeshVertex* ArcVerts = (FDynamicMeshVertex*)FMemory::Malloc( (NumRings+1) * sizeof(FDynamicMeshVertex) );

		for(int32 i=0; i<NumRings+1; i++)
		{
			FDynamicMeshVertex* ArcVert = &ArcVerts[i];

			float angle = ((float)i/NumRings) * UE_PI;

			// Note- unit sphere, so position always has mag of one. We can just use it for normal!			
			ArcVert->Position.X = 0.0f;
			ArcVert->Position.Y = FMath::Sin(angle);
			ArcVert->Position.Z = FMath::Cos(angle);

			ArcVert->SetTangents(
				FVector3f(1,0,0),
				FVector3f(0.0f,-ArcVert->Position.Z,ArcVert->Position.Y),
				ArcVert->Position
				);

			ArcVert->TextureCoordinate[0].X = 0.0f;
			ArcVert->TextureCoordinate[0].Y = ((float)i/NumRings);
		}

		// Then rotate this arc NumSides+1 times.
		for(int32 s=0; s<NumSides+1; s++)
		{
			FRotator3f ArcRotator(0, 360.f * (float)s/NumSides, 0);
			FRotationMatrix44f ArcRot( ArcRotator );
			float XTexCoord = ((float)s/NumSides);

			for(int32 v=0; v<NumRings+1; v++)
			{
				int32 VIx = (NumRings+1)*s + v;

				Verts[VIx].Position = ArcRot.TransformPosition( ArcVerts[v].Position );
				
				Verts[VIx].SetTangents(
					ArcRot.TransformVector( ArcVerts[v].TangentX.ToFVector3f()),
					ArcRot.TransformVector( ArcVerts[v].GetTangentY() ),
					ArcRot.TransformVector( ArcVerts[v].TangentZ.ToFVector3f())
					);

				Verts[VIx].TextureCoordinate[0].X = XTexCoord;
				Verts[VIx].TextureCoordinate[0].Y = ArcVerts[v].TextureCoordinate[0].Y;
			}
		}

		// Add all of the vertices we generated to the mesh builder.
		for(int32 VertIdx=0; VertIdx < NumVerts; VertIdx++)
		{
			MeshBuilder.AddVertex(Verts[VertIdx]);
		}
		
		// Add all of the triangles we generated to the mesh builder.
		for(int32 s=0; s<NumSides; s++)
		{
			int32 a0start = (s+0) * (NumRings+1);
			int32 a1start = (s+1) * (NumRings+1);

			for(int32 r=0; r<NumRings; r++)
			{
				MeshBuilder.AddTriangle(a0start + r + 0, a1start + r + 0, a0start + r + 1);
				MeshBuilder.AddTriangle(a1start + r + 0, a1start + r + 1, a0start + r + 1);
			}
		}

		// Free our local copy of verts and arc verts
		FMemory::Free(Verts);
		FMemory::Free(ArcVerts);
	}
	MeshBuilder.Draw(PDI, FScaleMatrix( Radii ) * FRotationMatrix(Orientation) * FTranslationMatrix( Center ), MaterialRenderProxy, DepthPriority,bDisableBackfaceCulling);
}

FVector CalcConeVert(float Angle1, float Angle2, float AzimuthAngle)
{
	float ang1 = FMath::Clamp<float>(Angle1, 0.01f, (float)UE_PI - 0.01f);
	float ang2 = FMath::Clamp<float>(Angle2, 0.01f, (float)UE_PI - 0.01f);

	float sinX_2 = FMath::Sin(0.5f * ang1);
	float sinY_2 = FMath::Sin(0.5f * ang2);

	float sinSqX_2 = sinX_2 * sinX_2;
	float sinSqY_2 = sinY_2 * sinY_2;

	float tanX_2 = FMath::Tan(0.5f * ang1);
	float tanY_2 = FMath::Tan(0.5f * ang2);


	float phi = FMath::Atan2(FMath::Sin(AzimuthAngle)*sinY_2, FMath::Cos(AzimuthAngle)*sinX_2);
	float sinPhi = FMath::Sin(phi);
	float cosPhi = FMath::Cos(phi);
	float sinSqPhi = sinPhi*sinPhi;
	float cosSqPhi = cosPhi*cosPhi;

	float rSq, r, Sqr, alpha, beta;

	rSq = sinSqX_2*sinSqY_2 / (sinSqX_2*sinSqPhi + sinSqY_2*cosSqPhi);
	r = FMath::Sqrt(rSq);
	Sqr = FMath::Sqrt(1 - rSq);
	alpha = r*cosPhi;
	beta = r*sinPhi;

	FVector ConeVert;

	ConeVert.X = (1 - 2 * rSq);
	ConeVert.Y = 2 * Sqr*alpha;
	ConeVert.Z = 2 * Sqr*beta;

	return ConeVert;
}

void BuildConeVerts(float Angle1, float Angle2, float Scale, float XOffset, uint32 NumSides, TArray<FDynamicMeshVertex>& OutVerts, TArray<uint32>& OutIndices)
{
	TArray<FVector> ConeVerts;
	ConeVerts.AddUninitialized(NumSides);

	for (uint32 i = 0; i < NumSides; i++)
	{
		float Fraction = (float)i / (float)(NumSides);
		float Azi = 2.f* UE_PI*Fraction;
		ConeVerts[i] = (CalcConeVert(Angle1, Angle2, Azi) * Scale) + FVector(XOffset,0,0);
	}

	for (uint32 i = 0; i < NumSides; i++)
	{
		// Normal of the current face 
		FVector TriTangentZ = ConeVerts[(i + 1) % NumSides] ^ ConeVerts[i]; // aka triangle normal
		FVector TriTangentY = ConeVerts[i];
		FVector TriTangentX = TriTangentZ ^ TriTangentY;


		FDynamicMeshVertex V0, V1, V2;

		V0.Position = FVector3f(0) + FVector3f(XOffset,0,0);
		V0.TextureCoordinate[0].X = 0.0f;
		V0.TextureCoordinate[0].Y = (float)i / NumSides;
		V0.SetTangents((FVector3f)TriTangentX, (FVector3f)TriTangentY, (FVector3f)FVector(-1, 0, 0));
		int32 I0 = OutVerts.Add(V0);

		V1.Position = (FVector3f)ConeVerts[i];
		V1.TextureCoordinate[0].X = 1.0f;
		V1.TextureCoordinate[0].Y = (float)i / NumSides;
		FVector TriTangentZPrev = ConeVerts[i] ^ ConeVerts[i == 0 ? NumSides - 1 : i - 1]; // Normal of the previous face connected to this face
		V1.SetTangents((FVector3f)TriTangentX, (FVector3f)TriTangentY, (FVector3f)(TriTangentZPrev + TriTangentZ).GetSafeNormal());
		int32 I1 = OutVerts.Add(V1);

		V2.Position = (FVector3f)ConeVerts[(i + 1) % NumSides];
		V2.TextureCoordinate[0].X = 1.0f;
		V2.TextureCoordinate[0].Y = (float)((i + 1) % NumSides) / NumSides;
		FVector TriTangentZNext = ConeVerts[(i + 2) % NumSides] ^ ConeVerts[(i + 1) % NumSides]; // Normal of the next face connected to this face
		V2.SetTangents((FVector3f)TriTangentX, (FVector3f)TriTangentY, (FVector3f)(TriTangentZNext + TriTangentZ).GetSafeNormal());
		int32 I2 = OutVerts.Add(V2);

		// Flip winding for negative scale
		if(Scale >= 0.f)
		{
			OutIndices.Add(I0);
			OutIndices.Add(I1);
			OutIndices.Add(I2);
		}
		else
		{
			OutIndices.Add(I0);
			OutIndices.Add(I2);
			OutIndices.Add(I1);
		}
	}
}

void DrawCone(FPrimitiveDrawInterface* PDI,const FMatrix& ConeToWorld, float Angle1, float Angle2, uint32 NumSides, bool bDrawSideLines, const FLinearColor& SideLineColor, const FMaterialRenderProxy* MaterialRenderProxy, uint8 DepthPriority)
{
	TArray<FDynamicMeshVertex> MeshVerts;
	TArray<uint32> MeshIndices;
	BuildConeVerts(Angle1, Angle2, 1.f, 0.f, NumSides, MeshVerts, MeshIndices);

	FDynamicMeshBuilder MeshBuilder(PDI->View->GetFeatureLevel());
	MeshBuilder.AddVertices(MeshVerts);
	MeshBuilder.AddTriangles(MeshIndices);
	MeshBuilder.Draw(PDI, ConeToWorld, MaterialRenderProxy, DepthPriority, 0.f);

	if(bDrawSideLines)
	{
		TArray<FVector> ConeVerts;
		ConeVerts.AddUninitialized(NumSides);

		// Draw lines down major directions
		for(int32 i=0; i<4; i++)
		{
			float Fraction = (float)i / (float)(4);
			float Azi = 2.f* UE_PI*Fraction;
			FVector ConeVert = CalcConeVert(Angle1, Angle2, Azi);
			PDI->DrawLine( ConeToWorld.GetOrigin(), ConeToWorld.TransformPosition(ConeVert), SideLineColor, DepthPriority );
		}
	}
}

void BuildDiscVerts(const FVector& Center, const FVector& XAxis, const FVector& YAxis, double Radius, uint32 Sides, TArray<FDynamicMeshVertex>& OutVerts, TArray<uint32>& OutIndices)
{
	const float	AngleDelta = 2.0f * UE_PI / Sides;
	FVector	LastVertex = Center + XAxis * Radius;
	FVector	ZAxis = XAxis ^ YAxis;

	FVector2D TC = FVector2D(0.0f, 0.0f);
	float TCStep = 1.0f / Sides;

	int32 BaseVertIndex = OutVerts.Num();

	for (uint32 SideIndex = 0; SideIndex < Sides; SideIndex++)
	{
		const FVector Vertex = Center + (XAxis * FMath::Cos(AngleDelta * (SideIndex + 1)) + YAxis * FMath::Sin(AngleDelta * (SideIndex + 1))) * Radius;
		FVector Normal = Vertex - Center;
		Normal.Normalize();

		FDynamicMeshVertex MeshVertex;

		MeshVertex.Position = FVector3f(Vertex);
		MeshVertex.TextureCoordinate[0] = FVector2f(TC);

		MeshVertex.SetTangents(
			(FVector3f)-ZAxis,
			FVector3f((-ZAxis) ^ Normal),
			(FVector3f)Normal
		);

		OutVerts.Add(MeshVertex); //Add bottom vertex

		LastVertex = Vertex;
		TC.X += TCStep;
	}

	//Add top triangles, in the style of a fan.
	//Note if we wanted nice rendering of the caps then we need to duplicate the vertices and modify
	//texture/tangent coordinates.
	for (uint32 SideIndex = 1; SideIndex < Sides; SideIndex++)
	{
		int32 V0 = BaseVertIndex;
		int32 V1 = BaseVertIndex + SideIndex;
		int32 V2 = BaseVertIndex + ((SideIndex + 1) % Sides);

		//bottom
		OutIndices.Add(V0);
		OutIndices.Add(V1);
		OutIndices.Add(V2);
	}
}

void GetDiscMesh(const FVector& Center, const FVector& XAxis, const FVector& YAxis, double Radius, int32 Sides, const FMaterialRenderProxy* MaterialRenderProxy, uint8 DepthPriority, int32 ViewIndex, FMeshElementCollector& Collector, HHitProxy* HitProxy)
{
	GetDiscMesh(FMatrix::Identity, Center, XAxis, YAxis, Radius, Sides, MaterialRenderProxy, DepthPriority, ViewIndex, Collector, HitProxy);
}

void GetDiscMesh(const FMatrix& ToWorld, const FVector& Center, const FVector& XAxis, const FVector& YAxis, double Radius, int32 Sides, const FMaterialRenderProxy* MaterialRenderProxy, uint8 DepthPriority, int32 ViewIndex, FMeshElementCollector& Collector, HHitProxy* HitProxy)
{
	TArray<FDynamicMeshVertex> MeshVerts;
	TArray<uint32> MeshIndices;
	BuildDiscVerts(Center, XAxis, YAxis, Radius, Sides, MeshVerts, MeshIndices);

	FDynamicMeshBuilder MeshBuilder(Collector.GetFeatureLevel());
	MeshBuilder.AddVertices(MeshVerts);
	MeshBuilder.AddTriangles(MeshIndices);

	MeshBuilder.GetMesh(ToWorld, MaterialRenderProxy, DepthPriority, false, false, true, ViewIndex, Collector, HitProxy);
}

void BuildCylinderVerts(const FVector& Base, const FVector& XAxis, const FVector& YAxis, const FVector& ZAxis, double Radius, double HalfHeight, uint32 Sides, TArray<FDynamicMeshVertex>& OutVerts, TArray<uint32>& OutIndices)
{
	// Call with equal top and bottom radius values
	BuildTaperedCylinderVerts(Base, XAxis, YAxis, ZAxis, Radius, Radius, HalfHeight, Sides, OutVerts, OutIndices);
}

void BuildTaperedCylinderVerts(const FVector& Base, const FVector& XAxis, const FVector& YAxis, const FVector& ZAxis, double BaseRadius, double TopRadius, double HalfHeight, uint32 Sides, TArray<FDynamicMeshVertex>& OutVerts, TArray<uint32>& OutIndices)
{
	const float	AngleDelta = 2.0f * UE_PI / Sides;
	FVector	LastVertex = Base + XAxis * BaseRadius;

	FVector2D TC = FVector2D(0.0f, 0.0f);
	float TCStep = 1.0f / Sides;

	FVector TopOffset = HalfHeight * ZAxis;

	int32 BaseVertIndex = OutVerts.Num();

	//Compute vertices for base circle.
	for (uint32 SideIndex = 0; SideIndex < Sides; SideIndex++)
	{
		const FVector Vertex = Base + (XAxis * FMath::Cos(AngleDelta * (SideIndex + 1)) + YAxis * FMath::Sin(AngleDelta * (SideIndex + 1))) * BaseRadius;
		FVector Normal = Vertex - Base;
		Normal.Normalize();

		FDynamicMeshVertex MeshVertex;

		MeshVertex.Position = FVector3f(Vertex - TopOffset);
		MeshVertex.TextureCoordinate[0] = FVector2f(TC);

		MeshVertex.SetTangents(
			(FVector3f)-ZAxis,
			FVector3f((-ZAxis) ^ Normal),
			(FVector3f)Normal
			);

		OutVerts.Add(MeshVertex); //Add bottom vertex

		LastVertex = Vertex;
		TC.X += TCStep;
	}

	LastVertex = Base + XAxis * BaseRadius;
	TC = FVector2D(0.0f, 1.0f);

	//Compute vertices for the top circle
	for (uint32 SideIndex = 0; SideIndex < Sides; SideIndex++)
	{
		const FVector Vertex = Base + (XAxis * FMath::Cos(AngleDelta * (SideIndex + 1)) + YAxis * FMath::Sin(AngleDelta * (SideIndex + 1))) * TopRadius;
		FVector Normal = Vertex - Base;
		Normal.Normalize();

		FDynamicMeshVertex MeshVertex;

		MeshVertex.Position = FVector3f(Vertex + TopOffset);
		MeshVertex.TextureCoordinate[0] = FVector2f(TC);

		MeshVertex.SetTangents(
			(FVector3f)-ZAxis,
			FVector3f((-ZAxis) ^ Normal),
			(FVector3f)Normal
			);

		OutVerts.Add(MeshVertex); //Add top vertex

		LastVertex = Vertex;
		TC.X += TCStep;
	}

	//Add top/bottom triangles, in the style of a fan.
	//Note if we wanted nice rendering of the caps then we need to duplicate the vertices and modify
	//texture/tangent coordinates.
	for (uint32 SideIndex = 1; SideIndex < Sides; SideIndex++)
	{
		int32 V0 = BaseVertIndex;
		int32 V1 = BaseVertIndex + SideIndex;
		int32 V2 = BaseVertIndex + ((SideIndex + 1) % Sides);

		//bottom
		OutIndices.Add(V0);
		OutIndices.Add(V1);
		OutIndices.Add(V2);

		// top
		OutIndices.Add(Sides + V2);
		OutIndices.Add(Sides + V1);
		OutIndices.Add(Sides + V0);
	}

	//Add sides.

	for (uint32 SideIndex = 0; SideIndex < Sides; SideIndex++)
	{
		int32 V0 = BaseVertIndex + SideIndex;
		int32 V1 = BaseVertIndex + ((SideIndex + 1) % Sides);
		int32 V2 = V0 + Sides;
		int32 V3 = V1 + Sides;

		OutIndices.Add(V0);
		OutIndices.Add(V2);
		OutIndices.Add(V1);

		OutIndices.Add(V2);
		OutIndices.Add(V3);
		OutIndices.Add(V1);
	}
}

void GetCylinderMesh(const FVector& Start, const FVector& End, double Radius, int32 Sides, const FMaterialRenderProxy* MaterialInstance, uint8 DepthPriority, int32 ViewIndex, FMeshElementCollector& Collector, HHitProxy* HitProxy)
{
	FVector Dir = End - Start;
	double Length = Dir.Size();

	if (Length > UE_SMALL_NUMBER)
	{
		FVector Z = Dir.GetUnsafeNormal();
		FVector X, Y;
		Z.GetUnsafeNormal().FindBestAxisVectors(X, Y);

		GetCylinderMesh(FMatrix::Identity, Z * Length*0.5 + Start, X, Y, Z, Radius, Length * 0.5f, Sides, MaterialInstance, DepthPriority, ViewIndex, Collector, HitProxy);
	}

}

void GetCylinderMesh(const FVector& Base, const FVector& XAxis, const FVector& YAxis, const FVector& ZAxis,
				  double Radius, double HalfHeight, int32 Sides, const FMaterialRenderProxy* MaterialRenderProxy, uint8 DepthPriority, int32 ViewIndex, FMeshElementCollector& Collector, HHitProxy* HitProxy)
{
	GetCylinderMesh( FMatrix::Identity, Base, XAxis, YAxis, ZAxis, Radius, HalfHeight, Sides, MaterialRenderProxy, DepthPriority, ViewIndex, Collector, HitProxy );
}

void GetCylinderMesh(const FMatrix& CylToWorld, const FVector& Base, const FVector& XAxis, const FVector& YAxis, const FVector& ZAxis, double Radius, double HalfHeight, uint32 Sides, const FMaterialRenderProxy* MaterialRenderProxy, uint8 DepthPriority, int32 ViewIndex, FMeshElementCollector& Collector, HHitProxy* HitProxy)
{
	TArray<FDynamicMeshVertex> MeshVerts;
	TArray<uint32> MeshIndices;
	BuildCylinderVerts(Base, XAxis, YAxis, ZAxis, Radius, HalfHeight, Sides, MeshVerts, MeshIndices);


	FDynamicMeshBuilder MeshBuilder(Collector.GetFeatureLevel());
	MeshBuilder.AddVertices(MeshVerts);
	MeshBuilder.AddTriangles(MeshIndices);

	MeshBuilder.GetMesh(CylToWorld, MaterialRenderProxy, DepthPriority, false, false, true, ViewIndex, Collector, HitProxy);
}

void GetConeMesh(const FMatrix& LocalToWorld, float AngleWidth, float AngleHeight, uint32 NumSides, const FMaterialRenderProxy* MaterialRenderProxy, uint8 DepthPriority, int32 ViewIndex, FMeshElementCollector& Collector, HHitProxy* HitProxy)
{
	TArray<FDynamicMeshVertex> MeshVerts;
	TArray<uint32> MeshIndices;
	BuildConeVerts(AngleWidth * UE_PI / 180, AngleHeight * UE_PI / 180, 1.f, 0.f, NumSides, MeshVerts, MeshIndices);
	FDynamicMeshBuilder MeshBuilder(Collector.GetFeatureLevel());
	MeshBuilder.AddVertices(MeshVerts);
	MeshBuilder.AddTriangles(MeshIndices);
	MeshBuilder.GetMesh(LocalToWorld, MaterialRenderProxy, DepthPriority, false, false, true, ViewIndex, Collector, HitProxy);
}

void GetCapsuleMesh(const FVector& Origin, const FVector& XAxis, const FVector& YAxis, const FVector& ZAxis, const FLinearColor& Color, double Radius, double HalfHeight, int32 NumSides, const FMaterialRenderProxy* MaterialRenderProxy, uint8 DepthPriority, bool bDisableBackfaceCulling, int32 ViewIndex, FMeshElementCollector& Collector, HHitProxy* HitProxy)
{
	const double HalfAxis = FMath::Max<double>(HalfHeight - Radius, 1.f);
	const FVector BottomEnd = Origin + Radius * ZAxis;
	const FVector TopEnd = BottomEnd + (2 * HalfAxis) * ZAxis;
	const double CylinderHalfHeight = (TopEnd - BottomEnd).Size() * 0.5;
	const FVector CylinderLocation = BottomEnd + CylinderHalfHeight * ZAxis;

	GetOrientedHalfSphereMesh(TopEnd, FRotationMatrix::MakeFromXY(XAxis, YAxis).Rotator(), FVector(Radius), NumSides, NumSides, 0, UE_PI / 2, MaterialRenderProxy, DepthPriority, bDisableBackfaceCulling, ViewIndex, Collector, false, HitProxy);
	GetCylinderMesh(CylinderLocation, XAxis, YAxis, ZAxis, Radius, CylinderHalfHeight, NumSides, MaterialRenderProxy, DepthPriority, ViewIndex, Collector, HitProxy);
	GetOrientedHalfSphereMesh(BottomEnd, FRotationMatrix::MakeFromXY(XAxis, YAxis).Rotator(), FVector(Radius), NumSides, NumSides, UE_PI / 2, UE_PI, MaterialRenderProxy, DepthPriority, bDisableBackfaceCulling, ViewIndex, Collector, false, HitProxy);
}

void DrawCylinder(FPrimitiveDrawInterface* PDI,const FVector& Base, const FVector& XAxis, const FVector& YAxis, const FVector& ZAxis,
	double Radius, double HalfHeight, uint32 Sides, const FMaterialRenderProxy* MaterialRenderProxy, uint8 DepthPriority)
{
	DrawCylinder( PDI, FMatrix::Identity, Base, XAxis, YAxis, ZAxis, Radius, HalfHeight, Sides, MaterialRenderProxy, DepthPriority );
}

void DrawCylinder(FPrimitiveDrawInterface* PDI, const FMatrix& CylToWorld, const FVector& Base, const FVector& XAxis, const FVector& YAxis, const FVector& ZAxis, double Radius, double HalfHeight, uint32 Sides, const FMaterialRenderProxy* MaterialRenderProxy, uint8 DepthPriority)
{
	TArray<FDynamicMeshVertex> MeshVerts;
	TArray<uint32> MeshIndices;
	BuildCylinderVerts(Base, XAxis, YAxis, ZAxis, Radius, HalfHeight, Sides, MeshVerts, MeshIndices);


	FDynamicMeshBuilder MeshBuilder(PDI->View->GetFeatureLevel());
	MeshBuilder.AddVertices(MeshVerts);
	MeshBuilder.AddTriangles(MeshIndices);

	MeshBuilder.Draw(PDI, CylToWorld, MaterialRenderProxy, DepthPriority,0.f);
}

void DrawCylinder(class FPrimitiveDrawInterface* PDI, const FVector& Start, const FVector& End, double Radius, int32 Sides, const FMaterialRenderProxy* MaterialInstance, uint8 DepthPriority)
{
	FVector Dir = End - Start;
	double Length = Dir.Size();

	if (Length > UE_SMALL_NUMBER)
	{
		FVector Z = Dir.GetUnsafeNormal();
		FVector X, Y;
		Z.GetUnsafeNormal().FindBestAxisVectors(X, Y);

		DrawCylinder(PDI, FMatrix::Identity, Z * Length*0.5 + Start, X, Y, Z, Radius, Length * 0.5f, Sides, MaterialInstance, DepthPriority);
	}
}

void DrawTorus(FPrimitiveDrawInterface* PDI, const FMatrix& Transform, const FVector& XAxis, const FVector& YAxis, double OuterRadius, double InnerRadius, int32 OuterSegments, int32 InnerSegments, const FMaterialRenderProxy* MaterialRenderProxy, uint8 DepthPriority, bool bPartial, float Angle, bool bEndCaps)
{
	DrawTorus(PDI, Transform, XAxis, YAxis, FColor::White, OuterRadius, InnerRadius, OuterSegments, InnerSegments, MaterialRenderProxy, DepthPriority, bPartial, Angle, bEndCaps);
}

void DrawTorus(FPrimitiveDrawInterface* PDI, const FMatrix& Transform, const FVector& XAxis, const FVector& YAxis, FColor Color,
			   double OuterRadius, double InnerRadius, int32 OuterSegments, int32 InnerSegments, const FMaterialRenderProxy* MaterialRenderProxy,
			   uint8 DepthPriority, bool bPartial, float Angle, bool bEndCaps)
{
	if (OuterSegments < 3)
	{
		OuterSegments = 3;
	}

	if (InnerSegments < 3)
	{
		InnerSegments = 3;
	}

	const float	AngleDelta = (bPartial ? Angle : 2.0f * UE_PI) / OuterSegments;
	FVector	LastVertex = XAxis * OuterRadius;

	int32 OuterSlices = bPartial ? (bEndCaps ? OuterSegments + 3 : OuterSegments + 1) : OuterSegments;
	const float InnerAngleDelta = 2.0f * UE_PI / InnerSegments;

	FVector ZAxis = XAxis ^ YAxis;
	ZAxis.Normalize();

	FVector2f TC = FVector2f(0.0f, 0.0f);
	float TCStepX = 1.0f / InnerSegments;
	float TCStepY = 1.0f / OuterSegments;

	FDynamicMeshBuilder MeshBuilder(PDI->View->GetFeatureLevel());

	for (int32 OuterIndex = 0; OuterIndex < OuterSlices; OuterIndex++)
	{
		float OuterAngle = AngleDelta * OuterIndex;

		FVector UnitDir = (XAxis * FMath::Cos(OuterAngle) + YAxis * FMath::Sin(OuterAngle));
		FVector InnerCenter = UnitDir * OuterRadius;

		TC.X = 0.0f;
		TC.Y = TCStepY * OuterIndex;

		bool bIsEndCapVertex = (bPartial && bEndCaps && (OuterIndex == 0 || OuterIndex == OuterSlices - 1));

		for (int32 InnerIndex = 0; InnerIndex < InnerSegments; InnerIndex++)
		{
			float InnerAngle = InnerAngleDelta * (InnerIndex + 1);

			FVector Vertex = InnerCenter;
			FVector Dir = UnitDir * FMath::Cos(InnerAngle) + ZAxis * FMath::Sin(InnerAngle);

			if (!bIsEndCapVertex)
			{
				Vertex += Dir * InnerRadius;
			}

			FDynamicMeshVertex MeshVertex;
			MeshVertex.Color = Color;
			MeshVertex.Position = FVector3f(Vertex);
			MeshVertex.TextureCoordinate[0] = TC;
			MeshVertex.TextureCoordinate[0].X += TCStepX * InnerIndex;

			// We invert the direction before deriving tangents, to counteract winding order
			// @see DrawSphere, etc.
			Dir *= -1.0;

			FVector TangentX = UnitDir * FMath::Sin(InnerAngle) + ZAxis * FMath::Cos(InnerAngle);
			FVector TangentY = Dir ^ TangentX;
			FVector TangentZ = Dir;
			MeshVertex.SetTangents((FVector3f)TangentX, (FVector3f)TangentY, (FVector3f)TangentZ);

			MeshBuilder.AddVertex(MeshVertex); //Add bottom vertex
		}
	}

	// Add sides.
	int32 NumVertices = OuterSlices * InnerSegments;
	int32 LoopSegments = bPartial ? OuterSlices - 1 : OuterSlices;

	for (int32 OuterIndex = 0; OuterIndex < LoopSegments; OuterIndex++)
	{
		int32 BaseVertexIndex = OuterIndex * InnerSegments;

		bool bIsEndCap = (bPartial && bEndCaps && (OuterIndex == 0 || OuterIndex == OuterSlices - 1));

		for (int32 InnerIndex = 0; InnerIndex < InnerSegments; InnerIndex++)
		{
			int32 V0 = BaseVertexIndex + InnerIndex;
			int32 V1 = BaseVertexIndex + ((InnerIndex + 1) % InnerSegments);
			int32 V2 = (V0 + InnerSegments) % NumVertices;
			int32 V3 = (V1 + InnerSegments) % NumVertices;

			if (bIsEndCap)
			{
				if (OuterIndex == 0)
				{
					MeshBuilder.AddTriangle(V0, V2, V3);
				}
				else
				{
					MeshBuilder.AddTriangle(V0, V1, V2);
				}
			}
			else
			{
				MeshBuilder.AddTriangle(V0, V2, V1);
				MeshBuilder.AddTriangle(V2, V3, V1);
			}
		}
	}

	MeshBuilder.Draw(PDI, Transform, MaterialRenderProxy, DepthPriority, 0.f);
}


void DrawDisc(class FPrimitiveDrawInterface* PDI,const FVector& Base,const FVector& XAxis,const FVector& YAxis,FColor Color,double Radius,int32 NumSides,const FMaterialRenderProxy* MaterialRenderProxy, uint8 DepthPriority)
{
	check (NumSides >= 3);

	const float	AngleDelta = 2.0f * UE_PI / NumSides;

	FVector2D TC = FVector2D(0.0f, 0.0f);
	float TCStep = 1.0f / NumSides;
	
	FVector ZAxis = (XAxis) ^ YAxis;

	FDynamicMeshBuilder MeshBuilder(PDI->View->GetFeatureLevel());

	//Compute vertices for base circle.
	for(int32 SideIndex = 0;SideIndex < NumSides;SideIndex++)
	{
		const FVector Vertex = Base + (XAxis * FMath::Cos(AngleDelta * (SideIndex)) + YAxis * FMath::Sin(AngleDelta * (SideIndex))) * Radius;
		FVector Normal = Vertex - Base;
		Normal.Normalize();

		FDynamicMeshVertex MeshVertex;
		MeshVertex.Position = (FVector3f)Vertex;
		MeshVertex.Color = Color;
		MeshVertex.TextureCoordinate[0] = FVector2f(TC);
		MeshVertex.TextureCoordinate[0].X += TCStep * SideIndex;

		MeshVertex.SetTangents(
			(FVector3f)-ZAxis,
			FVector3f((-ZAxis) ^ Normal),
			(FVector3f)Normal
			);

		MeshBuilder.AddVertex(MeshVertex); //Add bottom vertex
	}
	
	//Add top/bottom triangles, in the style of a fan.
	for(int32 SideIndex = 0; SideIndex < NumSides-1; SideIndex++)
	{
		int32 V0 = 0;
		int32 V1 = SideIndex;
		int32 V2 = (SideIndex + 1);

		MeshBuilder.AddTriangle(V0, V1, V2);
		MeshBuilder.AddTriangle(V0, V2, V1);
	}

	MeshBuilder.Draw(PDI, FMatrix::Identity, MaterialRenderProxy, DepthPriority,0.f);
}

void DrawFlatArrow(class FPrimitiveDrawInterface* PDI,const FVector& Base,const FVector& XAxis,const FVector& YAxis,FColor Color,float Length,int32 Width, const FMaterialRenderProxy* MaterialRenderProxy, uint8 DepthPriority, float Thickness)
{
	float DistanceFromBaseToHead = Length/3.0f;
	float DistanceFromBaseToTip = DistanceFromBaseToHead*2.0f;
	float WidthOfBase = Width;
	float WidthOfHead = 2*Width;

	FVector ArrowPoints[7];
	//base points
	ArrowPoints[0] = Base - YAxis*(WidthOfBase*.5f);
	ArrowPoints[1] = Base + YAxis*(WidthOfBase*.5f);
	//inner head
	ArrowPoints[2] = ArrowPoints[0] + XAxis*DistanceFromBaseToHead;
	ArrowPoints[3] = ArrowPoints[1] + XAxis*DistanceFromBaseToHead;
	//outer head
	ArrowPoints[4] = ArrowPoints[2] - YAxis*(WidthOfBase*.5f);
	ArrowPoints[5] = ArrowPoints[3] + YAxis*(WidthOfBase*.5f);
	//tip
	ArrowPoints[6] = Base + XAxis*Length;

	//Draw lines
	{
		const bool bThickLines = Thickness != 0.0f;
		const bool bDepthBiased = false;
		PDI->AddReserveLines(DepthPriority, 7, bDepthBiased, bThickLines);
		//base
		PDI->DrawLine(ArrowPoints[0], ArrowPoints[1], Color, DepthPriority, Thickness);
		//base sides
		PDI->DrawLine(ArrowPoints[0], ArrowPoints[2], Color, DepthPriority, Thickness);
		PDI->DrawLine(ArrowPoints[1], ArrowPoints[3], Color, DepthPriority, Thickness);
		//head base
		PDI->DrawLine(ArrowPoints[2], ArrowPoints[4], Color, DepthPriority, Thickness);
		PDI->DrawLine(ArrowPoints[3], ArrowPoints[5], Color, DepthPriority, Thickness);
		//head sides
		PDI->DrawLine(ArrowPoints[4], ArrowPoints[6], Color, DepthPriority, Thickness);
		PDI->DrawLine(ArrowPoints[5], ArrowPoints[6], Color, DepthPriority, Thickness);

	}

	FDynamicMeshBuilder MeshBuilder(PDI->View->GetFeatureLevel());

	//Compute vertices for base circle.
	for(int32 i = 0; i< 7; ++i)
	{
		FDynamicMeshVertex MeshVertex;
		MeshVertex.Position = (FVector3f)ArrowPoints[i];
		MeshVertex.Color = Color;
		MeshVertex.TextureCoordinate[0] = FVector2f(0.0f, 0.0f);;
		MeshVertex.SetTangents(FVector3f(XAxis^YAxis), (FVector3f)YAxis, (FVector3f)XAxis);
		MeshBuilder.AddVertex(MeshVertex); //Add bottom vertex
	}
	
	//Add triangles / double sided
	{
		MeshBuilder.AddTriangle(0, 2, 1); //base
		MeshBuilder.AddTriangle(0, 1, 2); //base
		MeshBuilder.AddTriangle(1, 2, 3); //base
		MeshBuilder.AddTriangle(1, 3, 2); //base
		MeshBuilder.AddTriangle(4, 5, 6); //head
		MeshBuilder.AddTriangle(4, 6, 5); //head
	}

	MeshBuilder.Draw(PDI, FMatrix::Identity, MaterialRenderProxy, DepthPriority, 0.f);
}

// Line drawing utility functions.

void DrawWireBox(FPrimitiveDrawInterface* PDI, const FBox& Box, const FLinearColor& Color, uint8 DepthPriority, float Thickness, float DepthBias, bool bScreenSpace)
{
	FVector	B[2],P,Q;
	int32 i,j;

	B[0]=Box.Min;
	B[1]=Box.Max;

	for( i=0; i<2; i++ ) for( j=0; j<2; j++ )
	{
		P.X=B[i].X; Q.X=B[i].X;
		P.Y=B[j].Y; Q.Y=B[j].Y;
		P.Z=B[0].Z; Q.Z=B[1].Z;
		PDI->DrawLine(P, Q, Color, DepthPriority, Thickness, DepthBias, bScreenSpace);

		P.Y=B[i].Y; Q.Y=B[i].Y;
		P.Z=B[j].Z; Q.Z=B[j].Z;
		P.X=B[0].X; Q.X=B[1].X;
		PDI->DrawLine(P, Q, Color, DepthPriority, Thickness, DepthBias, bScreenSpace);

		P.Z=B[i].Z; Q.Z=B[i].Z;
		P.X=B[j].X; Q.X=B[j].X;
		P.Y=B[0].Y; Q.Y=B[1].Y;
		PDI->DrawLine(P, Q, Color, DepthPriority, Thickness, DepthBias, bScreenSpace);
	}
}


void DrawWireBox(FPrimitiveDrawInterface* PDI, const FMatrix& Matrix, const FBox& Box, const FLinearColor& Color, uint8 DepthPriority, float Thickness, float DepthBias, bool bScreenSpace)
{
	FVector B[ 2 ];
	B[ 0 ] = Box.Min;
	B[ 1 ] = Box.Max;

	for( int i = 0; i < 2; i++ )
	{
		for( int j = 0; j < 2; j++ )
		{
			FVector P, Q;

			P.X = B[ i ].X; Q.X = B[ i ].X;
			P.Y = B[ j ].Y; Q.Y = B[ j ].Y;
			P.Z = B[ 0 ].Z; Q.Z = B[ 1 ].Z;
			P = Matrix.TransformPosition( P ); Q = Matrix.TransformPosition( Q );
			PDI->DrawLine(P, Q, Color, DepthPriority, Thickness, DepthBias, bScreenSpace);

			P.Y = B[ i ].Y; Q.Y = B[ i ].Y;
			P.Z = B[ j ].Z; Q.Z = B[ j ].Z;
			P.X = B[ 0 ].X; Q.X = B[ 1 ].X;
			P = Matrix.TransformPosition( P ); Q = Matrix.TransformPosition( Q );
			PDI->DrawLine(P, Q, Color, DepthPriority, Thickness, DepthBias, bScreenSpace);

			P.Z = B[ i ].Z; Q.Z = B[ i ].Z;
			P.X = B[ j ].X; Q.X = B[ j ].X;
			P.Y = B[ 0 ].Y; Q.Y = B[ 1 ].Y;
			P = Matrix.TransformPosition( P ); Q = Matrix.TransformPosition( Q );
			PDI->DrawLine(P, Q, Color, DepthPriority, Thickness, DepthBias, bScreenSpace);
		}
	}
}


void DrawCircle(FPrimitiveDrawInterface* PDI, const FVector& Base, const FVector& X, const FVector& Y, const FLinearColor& Color, double Radius, int32 NumSides, uint8 DepthPriority, float Thickness, float DepthBias, bool bScreenSpace)
{
	const float	AngleDelta = 2.0f * UE_PI / NumSides;
	FVector	LastVertex = Base + X * Radius;

	for(int32 SideIndex = 0; SideIndex < NumSides; SideIndex++)
	{
		const FVector Vertex = Base + (X * FMath::Cos(AngleDelta * (SideIndex + 1)) + Y * FMath::Sin(AngleDelta * (SideIndex + 1))) * Radius;
		PDI->DrawTranslucentLine(LastVertex, Vertex, Color, DepthPriority, Thickness, DepthBias, bScreenSpace);
		LastVertex = Vertex;
	}
}

void DrawArc(FPrimitiveDrawInterface* PDI, const FVector Base, const FVector X, const FVector Y, const float MinAngle, const float MaxAngle, const double Radius, const int32 Sections, const FLinearColor& Color, uint8 DepthPriority)
{
	float AngleStep = (MaxAngle - MinAngle)/((float)(Sections));
	float CurrentAngle = MinAngle;

	FVector LastVertex = Base + Radius * ( FMath::Cos(CurrentAngle * (UE_PI/180.0f)) * X + FMath::Sin(CurrentAngle * (UE_PI/180.0f)) * Y );
	CurrentAngle += AngleStep;

	for(int32 i=0; i<Sections; i++)
	{
		FVector ThisVertex = Base + Radius * ( FMath::Cos(CurrentAngle * (UE_PI/180.0f)) * X + FMath::Sin(CurrentAngle * (UE_PI/180.0f)) * Y );
		PDI->DrawTranslucentLine( LastVertex, ThisVertex, Color, DepthPriority );
		LastVertex = ThisVertex;
		CurrentAngle += AngleStep;
	}
}
void DrawRectangleMesh(FPrimitiveDrawInterface* PDI, const FVector& Center, const FVector& XAxis, const FVector& YAxis, 
					   FColor Color, float Width, float Height, const FMaterialRenderProxy* MaterialRenderProxy, uint8 DepthPriority)
{
	FVector XOffset = XAxis * Width * 0.5f;
	FVector YOffset = YAxis * Height * 0.5f;

	// Calculate verts for a face lying in plane defined by the X and Y vectors
	FVector Positions[4] =
	{
		Center - XOffset - YOffset,
		Center + XOffset - YOffset,
		Center + XOffset + YOffset,
		Center - XOffset + YOffset
	};

	FVector2f UVs[4] =
	{
		FVector2f(0.0f,0.0f),
		FVector2f(0.0f,1.0f),
		FVector2f(1.0f,1.0f),
		FVector2f(1.0f,0.0f),
	};

	FDynamicMeshBuilder MeshBuilder(PDI->View->GetFeatureLevel());

	int32 VertexIndices[4];
	for (int32 VertexIndex = 0; VertexIndex < 4; VertexIndex++)
	{
		VertexIndices[VertexIndex] = MeshBuilder.AddVertex(
			(FVector3f)Positions[VertexIndex],
			UVs[VertexIndex],
			FVector3f(1.0f, 0.0f, 0.0f),
			FVector3f(0.0f, 1.0f, 0.0f),
			FVector3f(0.0f, 0.0f, 1.0f),
			Color
		);
	}

	MeshBuilder.AddTriangle(VertexIndices[0], VertexIndices[1], VertexIndices[2]);
	MeshBuilder.AddTriangle(VertexIndices[0], VertexIndices[2], VertexIndices[3]);

	MeshBuilder.AddTriangle(VertexIndices[0], VertexIndices[2], VertexIndices[1]);
	MeshBuilder.AddTriangle(VertexIndices[0], VertexIndices[3], VertexIndices[2]);

	MeshBuilder.Draw(PDI, FMatrix::Identity, MaterialRenderProxy, DepthPriority, 0.f);
}

void DrawRectangle(class FPrimitiveDrawInterface* PDI, const FVector& Center, const FVector& XAxis, const FVector& YAxis, FColor Color, float Width, float Height, uint8 DepthPriority, float Thickness, float DepthBias, bool bScreenSpace)
{
	FVector XOffset = XAxis * Width * 0.5f;
	FVector YOffset = YAxis * Height * 0.5f;

	// Calculate verts for a face lying in plane defined by the X and Y vectors
	FVector Positions[4] =
	{
		Center - XOffset - YOffset,
		Center + XOffset - YOffset,
		Center + XOffset + YOffset,
		Center - XOffset + YOffset
	};

	FVector2D UVs[4] =
	{
		FVector2D(0.0f,0.0f),
		FVector2D(0.0f,1.0f),
		FVector2D(1.0f,1.0f),
		FVector2D(1.0f,0.0f),
	};

	PDI->DrawLine(Positions[0], Positions[1], Color, DepthPriority, Thickness, DepthBias, bScreenSpace);
	PDI->DrawLine(Positions[1], Positions[2], Color, DepthPriority, Thickness, DepthBias, bScreenSpace);
	PDI->DrawLine(Positions[2], Positions[3], Color, DepthPriority, Thickness, DepthBias, bScreenSpace);
	PDI->DrawLine(Positions[3], Positions[0], Color, DepthPriority, Thickness, DepthBias, bScreenSpace);
}

void DrawWireSphere(class FPrimitiveDrawInterface* PDI, const FVector& Base, const FLinearColor& Color, double Radius, int32 NumSides, uint8 DepthPriority, float Thickness, float DepthBias, bool bScreenSpace)
{
	DrawCircle(PDI, Base, FVector(1, 0, 0), FVector(0, 1, 0), Color, Radius, NumSides, DepthPriority, Thickness, DepthBias, bScreenSpace);
	DrawCircle(PDI, Base, FVector(1, 0, 0), FVector(0, 0, 1), Color, Radius, NumSides, DepthPriority, Thickness, DepthBias, bScreenSpace);
	DrawCircle(PDI, Base, FVector(0, 1, 0), FVector(0, 0, 1), Color, Radius, NumSides, DepthPriority, Thickness, DepthBias, bScreenSpace);
}

void DrawWireSphereAutoSides(class FPrimitiveDrawInterface* PDI, const FVector& Base, const FLinearColor& Color, double Radius, uint8 DepthPriority, float Thickness, float DepthBias, bool bScreenSpace)
{
	// Guess a good number of sides
	int32 NumSides =  FMath::Clamp<int32>(Radius/4.f, 16, 64);
	DrawWireSphere(PDI, Base, Color, Radius, NumSides, DepthPriority, Thickness, DepthBias, bScreenSpace);
}

void DrawWireSphere(class FPrimitiveDrawInterface* PDI, const FTransform& Transform, const FLinearColor& Color, double Radius, int32 NumSides, uint8 DepthPriority, float Thickness, float DepthBias, bool bScreenSpace)
{
	DrawCircle(PDI, Transform.GetLocation(), Transform.GetScaledAxis(EAxis::X), Transform.GetScaledAxis(EAxis::Y), Color, Radius, NumSides, DepthPriority, Thickness, DepthBias, bScreenSpace);
	DrawCircle(PDI, Transform.GetLocation(), Transform.GetScaledAxis(EAxis::X), Transform.GetScaledAxis(EAxis::Z), Color, Radius, NumSides, DepthPriority, Thickness, DepthBias, bScreenSpace);
	DrawCircle(PDI, Transform.GetLocation(), Transform.GetScaledAxis(EAxis::Y), Transform.GetScaledAxis(EAxis::Z), Color, Radius, NumSides, DepthPriority, Thickness, DepthBias, bScreenSpace);
}

void DrawWireSphereAutoSides(class FPrimitiveDrawInterface* PDI, const FTransform& Transform, const FLinearColor& Color, double Radius, uint8 DepthPriority, float Thickness, float DepthBias, bool bScreenSpace)
{
	// Guess a good number of sides
	int32 NumSides =  FMath::Clamp<int32>(Radius/4.f, 16, 64);
	DrawWireSphere(PDI, Transform, Color, Radius, NumSides, DepthPriority, Thickness, DepthBias, bScreenSpace);
}

void DrawWireCylinder(FPrimitiveDrawInterface* PDI, const FVector& Base, const FVector& X, const FVector& Y, const FVector& Z, const FLinearColor& Color, double Radius, double HalfHeight, int32 NumSides, uint8 DepthPriority, float Thickness, float DepthBias, bool bScreenSpace)
{
	const float	AngleDelta = 2.0f * UE_PI / NumSides;
	FVector	LastVertex = Base + X * Radius;

	for(int32 SideIndex = 0;SideIndex < NumSides;SideIndex++)
	{
		const FVector Vertex = Base + (X * FMath::Cos(AngleDelta * (SideIndex + 1)) + Y * FMath::Sin(AngleDelta * (SideIndex + 1))) * Radius;

		PDI->DrawLine(LastVertex - Z * HalfHeight, Vertex - Z * HalfHeight, Color, DepthPriority, Thickness, DepthBias, bScreenSpace);
		PDI->DrawLine(LastVertex + Z * HalfHeight, Vertex + Z * HalfHeight, Color, DepthPriority, Thickness, DepthBias, bScreenSpace);
		PDI->DrawLine(LastVertex - Z * HalfHeight, LastVertex + Z * HalfHeight, Color, DepthPriority, Thickness, DepthBias, bScreenSpace);

		LastVertex = Vertex;
	}
}

static void DrawHalfCircle(FPrimitiveDrawInterface* PDI, const FVector& Base, const FVector& X, const FVector& Y, const FLinearColor& Color, double Radius, int32 NumSides, uint8 DepthPriority, float Thickness, float DepthBias, bool bScreenSpace)
{
	const float	AngleDelta = (float)UE_PI / ((float)NumSides);
	FVector	LastVertex = Base + X * Radius;

	for(int32 SideIndex = 0; SideIndex < NumSides; SideIndex++)
	{
		const FVector	Vertex = Base + (X * FMath::Cos(AngleDelta * (SideIndex + 1)) + Y * FMath::Sin(AngleDelta * (SideIndex + 1))) * Radius;
		PDI->DrawTranslucentLine(LastVertex, Vertex, Color, DepthPriority, Thickness, DepthBias, bScreenSpace);
		LastVertex = Vertex;
	}	
}

void DrawWireCapsule(FPrimitiveDrawInterface* PDI, const FVector& Base, const FVector& X, const FVector& Y, const FVector& Z, const FLinearColor& Color, double Radius, double HalfHeight, int32 NumSides, uint8 DepthPriority, float Thickness, float DepthBias, bool bScreenSpace)
{
	const FVector Origin = Base;
	const FVector XAxis = X.GetSafeNormal();
	const FVector YAxis = Y.GetSafeNormal();
	const FVector ZAxis = Z.GetSafeNormal();

	// because we are drawing a capsule we have to have room for the "domed caps"
	const double XScale = X.Size();
	const double YScale = Y.Size();
	const double ZScale = Z.Size();
	double CapsuleRadius = Radius * FMath::Max(XScale,YScale);
	HalfHeight *= ZScale;
	CapsuleRadius = FMath::Clamp(CapsuleRadius, 0.f, HalfHeight);	//cap radius based on total height
	HalfHeight -= CapsuleRadius;
	HalfHeight = FMath::Max( 0.0f, HalfHeight );

	// Draw top and bottom circles
	const FVector TopEnd = Origin + (HalfHeight * ZAxis);
	const FVector BottomEnd = Origin - (HalfHeight * ZAxis);

	DrawCircle(PDI, TopEnd, XAxis, YAxis, Color, CapsuleRadius, NumSides, DepthPriority, Thickness, DepthBias, bScreenSpace);
	DrawCircle(PDI, BottomEnd, XAxis, YAxis, Color, CapsuleRadius, NumSides, DepthPriority, Thickness, DepthBias, bScreenSpace);

	// Draw domed caps
	DrawHalfCircle(PDI, TopEnd, YAxis, ZAxis, Color, CapsuleRadius, NumSides / 2, DepthPriority, Thickness, DepthBias, bScreenSpace);
	DrawHalfCircle(PDI, TopEnd, XAxis, ZAxis, Color, CapsuleRadius, NumSides / 2, DepthPriority, Thickness, DepthBias, bScreenSpace);

	const FVector NegZAxis = -ZAxis;

	DrawHalfCircle(PDI, BottomEnd, YAxis, NegZAxis, Color, CapsuleRadius, NumSides / 2, DepthPriority, Thickness, DepthBias, bScreenSpace);
	DrawHalfCircle(PDI, BottomEnd, XAxis, NegZAxis, Color, CapsuleRadius, NumSides / 2, DepthPriority, Thickness, DepthBias, bScreenSpace);

	// we set NumSides to 4 as it makes a nicer looking capsule as we only draw 2 HalfCircles above
	const int32 NumCylinderLines = 4;

	// Draw lines for the cylinder portion 
	const float	AngleDelta = 2.0f * UE_PI / NumCylinderLines;
	FVector	LastVertex = Base + XAxis * CapsuleRadius;

	for( int32 SideIndex = 0; SideIndex < NumCylinderLines; SideIndex++ )
	{
		const FVector Vertex = Base + (XAxis * FMath::Cos(AngleDelta * (SideIndex + 1)) + YAxis * FMath::Sin(AngleDelta * (SideIndex + 1))) * CapsuleRadius;

		PDI->DrawTranslucentLine(LastVertex - ZAxis * HalfHeight, LastVertex + ZAxis * HalfHeight, Color, DepthPriority, Thickness, DepthBias, bScreenSpace);

		LastVertex = Vertex;
	}
}

void DrawWireCone(FPrimitiveDrawInterface* PDI, TArray<FVector>& Verts, const FTransform& Transform, double ConeLength, double ConeAngle, int32 ConeSides, const FLinearColor& Color, uint8 DepthPriority, float Thickness, float DepthBias, bool bScreenSpace)
{
	static const float TwoPI = 2.0f * UE_PI;
	static const float ToRads = UE_PI / 180.0f;
	static const float MaxAngle = 89.0f * ToRads + 0.001f;
	const float ClampedConeAngle = FMath::Clamp(ConeAngle * ToRads, 0.001f, MaxAngle);
	const float SinClampedConeAngle = FMath::Sin( ClampedConeAngle );
	const float CosClampedConeAngle = FMath::Cos( ClampedConeAngle );
	const FVector ConeDirection(1,0,0);
	const FVector ConeUpVector(0,1,0);
	const FVector ConeLeftVector(0,0,1);

	Verts.AddUninitialized( ConeSides );

	for ( int32 i = 0 ; i < Verts.Num() ; ++i )
	{
		const float Theta = static_cast<float>( (TwoPI * i) / Verts.Num() );
		Verts[i] = (ConeDirection * (ConeLength * CosClampedConeAngle)) +
			((SinClampedConeAngle * ConeLength * FMath::Cos( Theta )) * ConeUpVector) +
			((SinClampedConeAngle * ConeLength * FMath::Sin( Theta )) * ConeLeftVector);
	}

	// Transform to world space.
	for ( int32 i = 0 ; i < Verts.Num() ; ++i )
	{
		Verts[i] = Transform.TransformPosition( Verts[i] );
	}

	// Draw spokes.
	for ( int32 i = 0 ; i < Verts.Num(); ++i )
	{
		PDI->DrawTranslucentLine(Transform.GetLocation(), Verts[i], Color, DepthPriority, Thickness, DepthBias, bScreenSpace);
	}

	// Draw rim.
	for ( int32 i = 0 ; i < Verts.Num()-1 ; ++i )
	{
		PDI->DrawTranslucentLine(Verts[i], Verts[i + 1], Color, DepthPriority, Thickness, DepthBias, bScreenSpace);
	}
	PDI->DrawTranslucentLine(Verts[Verts.Num() - 1], Verts[0], Color, DepthPriority, Thickness, DepthBias, bScreenSpace);
}

void DrawWireCone(FPrimitiveDrawInterface* PDI, TArray<FVector>& Verts, const FMatrix& Transform, double ConeLength, double ConeAngle, int32 ConeSides, const FLinearColor& Color, uint8 DepthPriority, float Thickness, float DepthBias, bool bScreenSpace)
{
	DrawWireCone(PDI, Verts, FTransform(Transform), ConeLength, ConeAngle, ConeSides, Color, DepthPriority, Thickness, DepthBias, bScreenSpace);
}

void DrawWireSphereCappedCone(FPrimitiveDrawInterface* PDI, const FTransform& Transform, double ConeLength, double ConeAngle, int32 ConeSides, int32 ArcFrequency, int32 CapSegments, const FLinearColor& Color, uint8 DepthPriority)
{
	// The cap only works if there are an even number of verts generated so add another if needed 
	if ((ConeSides & 0x1) != 0)
	{
		++ConeSides;
	}

	TArray<FVector> Verts;
	DrawWireCone(PDI, Verts, Transform, ConeLength, ConeAngle, ConeSides, Color, DepthPriority, 0);

	// Draw arcs
	int32 ArcCount = (int32)(Verts.Num() / 2);
	for ( int32 i = 0; i < ArcCount; i += ArcFrequency )
	{
		const FVector X = Transform.GetUnitAxis( EAxis::X );
		FVector Y = Verts[i] - Verts[ArcCount + i]; Y.Normalize();

		DrawArc(PDI, Transform.GetTranslation(), X, Y, -ConeAngle, ConeAngle, ConeLength, CapSegments, Color, DepthPriority);
	}
}


void DrawWireChoppedCone(FPrimitiveDrawInterface* PDI,const FVector& Base,const FVector& X,const FVector& Y,const FVector& Z,const FLinearColor& Color,double Radius, double TopRadius,double HalfHeight,int32 NumSides,uint8 DepthPriority)
{
	const float	AngleDelta = 2.0f * UE_PI / NumSides;
	FVector	LastVertex = Base + X * Radius;
	FVector LastTopVertex = Base + X * TopRadius;

	for(int32 SideIndex = 0;SideIndex < NumSides;SideIndex++)
	{
		const FVector Vertex = Base + (X * FMath::Cos(AngleDelta * (SideIndex + 1)) + Y * FMath::Sin(AngleDelta * (SideIndex + 1))) * Radius;
		const FVector TopVertex = Base + (X * FMath::Cos(AngleDelta * (SideIndex + 1)) + Y * FMath::Sin(AngleDelta * (SideIndex + 1))) * TopRadius;	

		PDI->DrawLine(LastVertex - Z * HalfHeight,Vertex - Z * HalfHeight,Color,DepthPriority);
		PDI->DrawLine(LastTopVertex + Z * HalfHeight,TopVertex + Z * HalfHeight,Color,DepthPriority);
		PDI->DrawLine(LastVertex - Z * HalfHeight,LastTopVertex + Z * HalfHeight,Color,DepthPriority);

		LastVertex = Vertex;
		LastTopVertex = TopVertex;
	}
}

void DrawOrientedWireBox(FPrimitiveDrawInterface* PDI, const FVector& Base, const FVector& X, const FVector& Y, const FVector& Z, FVector Extent, const FLinearColor& Color, uint8 DepthPriority, float Thickness, float DepthBias, bool bScreenSpace)
{
	FVector	B[2],P,Q;

	FMatrix M(X, Y, Z, Base);
	B[0] = -Extent;
	B[1] = Extent;

	for (int32 i = 0; i < 2; i++)
	{
		for (int32 j = 0; j < 2; j++)
		{
			P.X = B[i].X; Q.X = B[i].X;
			P.Y = B[j].Y; Q.Y = B[j].Y;
			P.Z = B[0].Z; Q.Z = B[1].Z;
			P = M.TransformPosition(P);
			Q = M.TransformPosition(Q);
			PDI->DrawTranslucentLine(P, Q, Color, DepthPriority, Thickness, DepthBias, bScreenSpace);

			P.Y = B[i].Y; Q.Y = B[i].Y;
			P.Z = B[j].Z; Q.Z = B[j].Z;
			P.X = B[0].X; Q.X = B[1].X;
			P = M.TransformPosition(P);
			Q = M.TransformPosition(Q);
			PDI->DrawTranslucentLine(P, Q, Color, DepthPriority, Thickness, DepthBias, bScreenSpace);

			P.Z = B[i].Z; Q.Z = B[i].Z;
			P.X = B[j].X; Q.X = B[j].X;
			P.Y = B[0].Y; Q.Y = B[1].Y;
			P = M.TransformPosition(P);
			Q = M.TransformPosition(Q);
			PDI->DrawTranslucentLine(P, Q, Color, DepthPriority, Thickness, DepthBias, bScreenSpace);
		}
	}
}


void DrawCoordinateSystem(FPrimitiveDrawInterface* PDI, FVector const& AxisLoc, FRotator const& AxisRot, float Scale, uint8 DepthPriority, float Thickness)
{
	FRotationMatrix R(AxisRot);
	FVector const X = R.GetScaledAxis( EAxis::X );
	FVector const Y = R.GetScaledAxis( EAxis::Y );
	FVector const Z = R.GetScaledAxis( EAxis::Z );

	PDI->DrawLine(AxisLoc, AxisLoc + X*Scale, FLinearColor::Red, DepthPriority, Thickness );
	PDI->DrawLine(AxisLoc, AxisLoc + Y*Scale, FLinearColor::Green, DepthPriority, Thickness );
	PDI->DrawLine(AxisLoc, AxisLoc + Z*Scale, FLinearColor::Blue, DepthPriority, Thickness );
}


void DrawCoordinateSystem(FPrimitiveDrawInterface* PDI, FVector const& AxisLoc, FRotator const& AxisRot, float Scale, const FLinearColor& InColor, uint8 DepthPriority, float Thickness)
{
	FRotationMatrix R(AxisRot);
	FVector const X = R.GetScaledAxis(EAxis::X);
	FVector const Y = R.GetScaledAxis(EAxis::Y);
	FVector const Z = R.GetScaledAxis(EAxis::Z);

	PDI->DrawLine(AxisLoc, AxisLoc + X * Scale, InColor, DepthPriority, Thickness);
	PDI->DrawLine(AxisLoc, AxisLoc + Y * Scale, InColor, DepthPriority, Thickness);
	PDI->DrawLine(AxisLoc, AxisLoc + Z * Scale, InColor, DepthPriority, Thickness);
}


void DrawDirectionalArrow(FPrimitiveDrawInterface* PDI,const FMatrix& ArrowToWorld,const FLinearColor& InColor,float Length,float ArrowSize,uint8 DepthPriority,float Thickness)
{
	PDI->DrawLine(ArrowToWorld.TransformPosition(FVector(Length,0,0)),ArrowToWorld.TransformPosition(FVector::ZeroVector),InColor,DepthPriority,Thickness);
	PDI->DrawLine(ArrowToWorld.TransformPosition(FVector(Length,0,0)),ArrowToWorld.TransformPosition(FVector(Length-ArrowSize,+ArrowSize,+ArrowSize)),InColor,DepthPriority,Thickness);
	PDI->DrawLine(ArrowToWorld.TransformPosition(FVector(Length,0,0)),ArrowToWorld.TransformPosition(FVector(Length-ArrowSize,+ArrowSize,-ArrowSize)),InColor,DepthPriority,Thickness);
	PDI->DrawLine(ArrowToWorld.TransformPosition(FVector(Length,0,0)),ArrowToWorld.TransformPosition(FVector(Length-ArrowSize,-ArrowSize,+ArrowSize)),InColor,DepthPriority,Thickness);
	PDI->DrawLine(ArrowToWorld.TransformPosition(FVector(Length,0,0)),ArrowToWorld.TransformPosition(FVector(Length-ArrowSize,-ArrowSize,-ArrowSize)),InColor,DepthPriority,Thickness);
}

void DrawConnectedArrow(class FPrimitiveDrawInterface* PDI, const FMatrix& ArrowToWorld, const FLinearColor& Color, float ArrowHeight, float ArrowWidth, uint8 DepthPriority, float Thickness, int32 NumSpokes)
{
	float RotPerSpoke = (2.0f * UE_PI) / (float)NumSpokes;
	FQuat Rotator(FVector(1.0f, 0.0f, 0.0f), RotPerSpoke);

	FVector Origin = ArrowToWorld.GetOrigin();
	FVector SpokePoint = FVector(-ArrowHeight, ArrowWidth, 0);
	for(int32 CurrSpoke = 0 ; CurrSpoke < NumSpokes ; ++CurrSpoke)
	{
		PDI->DrawLine(Origin, ArrowToWorld.TransformPosition(SpokePoint), Color, DepthPriority, Thickness);
		FVector PrevPoint = SpokePoint;
		SpokePoint = Rotator.RotateVector(SpokePoint);
		PDI->DrawLine(ArrowToWorld.TransformPosition(PrevPoint), ArrowToWorld.TransformPosition(SpokePoint), Color, DepthPriority, Thickness);
	}
}

void DrawWireStar(FPrimitiveDrawInterface* PDI,const FVector& Position, float Size, const FLinearColor& Color,uint8 DepthPriority)
{
	PDI->DrawLine(Position + Size * FVector(1,0,0), Position - Size * FVector(1,0,0), Color, DepthPriority);
	PDI->DrawLine(Position + Size * FVector(0,1,0), Position - Size * FVector(0,1,0), Color, DepthPriority);
	PDI->DrawLine(Position + Size * FVector(0,0,1), Position - Size * FVector(0,0,1), Color, DepthPriority);
}

void DrawDashedLine(FPrimitiveDrawInterface* PDI, const FVector& Start, const FVector& End, const FLinearColor& Color, double DashSize, uint8 DepthPriority, float DepthBias)
{
	FVector LineDir = End - Start;
	double LineLeft = (End - Start).Size();
	if (LineLeft)
	{
		LineDir /= LineLeft;
	}

	const int32 nLines = FMath::CeilToInt32(LineLeft / (DashSize*2));
	PDI->AddReserveLines(DepthPriority, nLines, DepthBias != 0);

	const FVector Dash = (DashSize * LineDir);

	FVector DrawStart = Start;
	while (LineLeft > DashSize)
	{
		const FVector DrawEnd = DrawStart + Dash;

		PDI->DrawLine(DrawStart, DrawEnd, Color, DepthPriority, 0.0f, DepthBias);

		LineLeft -= 2*DashSize;
		DrawStart = DrawEnd + Dash;
	}
	if (LineLeft > 0.0f)
	{
		const FVector DrawEnd = End;

		PDI->DrawLine(DrawStart, DrawEnd, Color, DepthPriority, 0.0f, DepthBias);
	}
}

void DrawWireDiamond(FPrimitiveDrawInterface* PDI,const FMatrix& DiamondMatrix, float Size, const FLinearColor& InColor,uint8 DepthPriority, float Thickness)
{
	const FVector TopPoint = DiamondMatrix.TransformPosition( FVector(0,0,1) * Size );
	const FVector BottomPoint = DiamondMatrix.TransformPosition( FVector(0,0,-1) * Size );

	const float OneOverRootTwo = FMath::Sqrt(0.5f);

	FVector SquarePoints[4];
	SquarePoints[0] = DiamondMatrix.TransformPosition( FVector(1,1,0) * Size * OneOverRootTwo );
	SquarePoints[1] = DiamondMatrix.TransformPosition( FVector(1,-1,0) * Size * OneOverRootTwo );
	SquarePoints[2] = DiamondMatrix.TransformPosition( FVector(-1,-1,0) * Size * OneOverRootTwo );
	SquarePoints[3] = DiamondMatrix.TransformPosition( FVector(-1,1,0) * Size * OneOverRootTwo );

	PDI->DrawLine(TopPoint, SquarePoints[0], InColor, DepthPriority, Thickness);
	PDI->DrawLine(TopPoint, SquarePoints[1], InColor, DepthPriority, Thickness);
	PDI->DrawLine(TopPoint, SquarePoints[2], InColor, DepthPriority, Thickness);
	PDI->DrawLine(TopPoint, SquarePoints[3], InColor, DepthPriority, Thickness);

	PDI->DrawLine(BottomPoint, SquarePoints[0], InColor, DepthPriority, Thickness);
	PDI->DrawLine(BottomPoint, SquarePoints[1], InColor, DepthPriority, Thickness);
	PDI->DrawLine(BottomPoint, SquarePoints[2], InColor, DepthPriority, Thickness);
	PDI->DrawLine(BottomPoint, SquarePoints[3], InColor, DepthPriority, Thickness);

	PDI->DrawLine(SquarePoints[0], SquarePoints[1], InColor, DepthPriority, Thickness);
	PDI->DrawLine(SquarePoints[1], SquarePoints[2], InColor, DepthPriority, Thickness);
	PDI->DrawLine(SquarePoints[2], SquarePoints[3], InColor, DepthPriority, Thickness);
	PDI->DrawLine(SquarePoints[3], SquarePoints[0], InColor, DepthPriority, Thickness);
}

static FLinearColor ApplySelectionIntensity(const FLinearColor& FinalColor, bool bSelected, bool bHovered, bool bUseOverlayIntensity)
{
	static const float BaseIntensity = 0.5f;
	static const float SelectedIntensity = 0.5f;
	static const float HoverIntensity = 0.15f;

	const float OverlayIntensity = bUseOverlayIntensity ? GEngine->SelectionHighlightIntensity : 1.0f;
	float ResultingIntensity = bSelected ? SelectedIntensity : (bHovered ? HoverIntensity : 0.0f);

	ResultingIntensity = (ResultingIntensity * OverlayIntensity) + BaseIntensity;

	FLinearColor ret = FinalColor * FMath::Pow(ResultingIntensity, 2.2f);
	ret.A = FinalColor.A;

	return ret;
}

FLinearColor GetSelectionColor(const FLinearColor& BaseColor,bool bSelected,bool bHovered, bool bUseOverlayIntensity)
{
	FLinearColor FinalColor = BaseColor;
	if( bSelected )
	{
		FinalColor = GEngine->GetSelectedMaterialColor();
	}

	return ApplySelectionIntensity(FinalColor, bSelected, bHovered, bUseOverlayIntensity );

}

FLinearColor GetViewSelectionColor(const FLinearColor& BaseColor, const FSceneView& View, bool bSelected, bool bHovered, bool bUseOverlayIntensity, bool bIndividuallySelected)
{
	FLinearColor FinalColor = BaseColor;
#if WITH_EDITOR
	if (View.bHasSelectedComponents && !bIndividuallySelected)
	{
		FinalColor = GEngine->GetSubduedSelectionOutlineColor();
	}
	else if( bSelected )
	{
		FinalColor = GEngine->GetSelectedMaterialColor();
	}
#endif

	return ApplySelectionIntensity(FinalColor, bSelected, bHovered, bUseOverlayIntensity );
}

bool IsRichView(const FSceneViewFamily& ViewFamily)
{
	// Flags which make the view rich when absent.
	if( !ViewFamily.EngineShowFlags.LOD ||
		// Force FDrawBasePassDynamicMeshAction to be used since it has access to the view and can implement the show flags
		!ViewFamily.EngineShowFlags.VolumetricLightmap ||
		!ViewFamily.EngineShowFlags.IndirectLightingCache ||
		(!ViewFamily.EngineShowFlags.Lighting && ViewFamily.EngineShowFlags.UnlitRichView) ||
		!ViewFamily.EngineShowFlags.Materials)
	{
		return true;
	}

	// Flags which make the view rich when present.
	if( ViewFamily.UseDebugViewPS()	||
		ViewFamily.EngineShowFlags.LightComplexity ||
		ViewFamily.EngineShowFlags.StationaryLightOverlap ||
		ViewFamily.EngineShowFlags.BSPSplit ||
		ViewFamily.EngineShowFlags.LightMapDensity ||
		ViewFamily.EngineShowFlags.MeshEdges ||
		ViewFamily.EngineShowFlags.LightInfluences ||
		ViewFamily.EngineShowFlags.Wireframe ||
		ViewFamily.EngineShowFlags.ActorColoration ||
		ViewFamily.EngineShowFlags.LODColoration ||
		ViewFamily.EngineShowFlags.HLODColoration ||
		ViewFamily.EngineShowFlags.MassProperties )
	{
		return true;
	}

	return false;
}

void ApplyViewModeOverrides(
	int32 ViewIndex,
	const FEngineShowFlags& EngineShowFlags,
	ERHIFeatureLevel::Type FeatureLevel,
	const FPrimitiveSceneProxy* PrimitiveSceneProxy,
	bool bSelected,
	FMeshBatch& Mesh,
	FMeshElementCollector& Collector
	)
{
#if !UE_BUILD_TEST

	// If debug viewmodes are not allowed, skip all of the debug viewmode handling.
	if (!AllowDebugViewmodes())
	{
		return;
	}

	const bool bMaterialModifiesMeshPosition = Mesh.MaterialRenderProxy->GetIncompleteMaterialWithFallback(FeatureLevel).MaterialModifiesMeshPosition_RenderThread();

	if (EngineShowFlags.Wireframe)
	{
		// In wireframe mode, draw the edges of the mesh with the specified wireframe color, or
		// with the level color if level coloration is enabled.
		FLinearColor BaseColor( PrimitiveSceneProxy->GetWireframeColor() );
		if (EngineShowFlags.ActorColoration)
		{
			BaseColor = PrimitiveSceneProxy->GetPrimitiveColor();
		}

		if (bMaterialModifiesMeshPosition)
		{
			// If the material is mesh-modifying, we cannot rely on substitution.
			auto WireframeMaterialInstance = new FOverrideSelectionColorMaterialRenderProxy(
				Mesh.MaterialRenderProxy,
				GetSelectionColor(BaseColor, bSelected, PrimitiveSceneProxy->IsHovered(), /*bUseOverlayIntensity=*/false)
			);

			Mesh.bWireframe = true;
			Mesh.MaterialRenderProxy = WireframeMaterialInstance;
			Collector.RegisterOneFrameMaterialProxy(WireframeMaterialInstance);
		}
		else
		{
			auto WireframeMaterialInstance = new FColoredMaterialRenderProxy(
				GEngine->WireframeMaterial->GetRenderProxy(),
				GetSelectionColor(BaseColor, bSelected, PrimitiveSceneProxy->IsHovered(), /*bUseOverlayIntensity=*/false)
			);

			Mesh.bWireframe = true;
			Mesh.MaterialRenderProxy = WireframeMaterialInstance;
			Collector.RegisterOneFrameMaterialProxy(WireframeMaterialInstance);
		}
	}
	else if (!EngineShowFlags.Materials)
	{
		// Don't render unlit translucency when in 'lighting only' viewmode.
		const FMaterial& Mat = Mesh.MaterialRenderProxy->GetIncompleteMaterialWithFallback(FeatureLevel);
		if (Mat.GetShadingModels().IsLit()
			// Don't render translucency in 'lighting only', since the viewmode works by overriding with an opaque material
			// This would cause a mismatch of the material's blend mode with the primitive's view relevance,
			// And make faint particles block the view
			&& !IsTranslucentBlendMode(Mat.GetBlendMode()))
		{
			// When materials aren't shown, apply the same basic material to all meshes.
			bool bTextureMapped = false;
			FVector2D LMResolution;

			if (EngineShowFlags.LightMapDensity && Mesh.LCI)
			{
				auto Interaction = Mesh.LCI->GetLightMapInteraction(FeatureLevel);
				auto Texture = Interaction.GetTexture(AllowHighQualityLightmaps(FeatureLevel));

				if (Interaction.GetType() == LMIT_Texture && Texture)
				{
					LMResolution.X = Texture->GetSizeX();
					LMResolution.Y = Texture->GetSizeY();
					bTextureMapped = true;
				}
			}

			if (bTextureMapped == false)
			{
				FMaterialRenderProxy* RenderProxy = GEngine->LevelColorationLitMaterial->GetRenderProxy();
				auto LightingOnlyMaterialInstance = new FColoredMaterialRenderProxy(
					RenderProxy,
					GEngine->LightingOnlyBrightness
				);

				Mesh.MaterialRenderProxy = LightingOnlyMaterialInstance;
				Collector.RegisterOneFrameMaterialProxy(LightingOnlyMaterialInstance);
			}
			else
			{
				FMaterialRenderProxy* RenderProxy = GEngine->LightingTexelDensityMaterial->GetRenderProxy();
				auto LightingDensityMaterialInstance = new FLightingDensityMaterialRenderProxy(
					RenderProxy,
					GEngine->LightingOnlyBrightness,
					LMResolution
					);

				Mesh.MaterialRenderProxy = LightingDensityMaterialInstance;
				Collector.RegisterOneFrameMaterialProxy(LightingDensityMaterialInstance);
			}
		}
	}
	else
	{	
		if (EngineShowFlags.ActorColoration)
		{
			const FLinearColor SelectionColor = GetSelectionColor(PrimitiveSceneProxy->GetPrimitiveColor(), bSelected, PrimitiveSceneProxy->IsHovered());
			FMaterialRenderProxy* LevelColorationMaterialInstance = nullptr;

			if (bMaterialModifiesMeshPosition)
			{
				// If the material is mesh-modifying, we cannot rely on substitution.
				LevelColorationMaterialInstance = new FOverrideSelectionColorMaterialRenderProxy(Mesh.MaterialRenderProxy, SelectionColor);
			}
			else
			{
				const UMaterial* LevelColorationMaterial = EngineShowFlags.Lighting ? GEngine->LevelColorationLitMaterial : GEngine->LevelColorationUnlitMaterial;
				// Draw the mesh with level coloration.
				LevelColorationMaterialInstance = new FColoredMaterialRenderProxy(LevelColorationMaterial->GetRenderProxy(), SelectionColor);
			}

			Mesh.MaterialRenderProxy = LevelColorationMaterialInstance;
			Collector.RegisterOneFrameMaterialProxy(LevelColorationMaterialInstance);
		}
#if WITH_EDITOR
		else if (EngineShowFlags.RandomColor)
		{
			FRandomStream RandomStream(GetTypeHash(PrimitiveSceneProxy->GetPrimitiveComponentId().PrimIDValue) + GetTypeHash(Mesh.MeshIdInPrimitive));

			const uint8 R = static_cast<uint8>(RandomStream.FRandRange(64, 255));
			const uint8 G = static_cast<uint8>(RandomStream.FRandRange(64, 255));
			const uint8 B = static_cast<uint8>(RandomStream.FRandRange(64, 255));
			FLinearColor RandomColor(FColor(R, G, B, 255));
			FMaterialRenderProxy* LevelColorationMaterialInstance = nullptr;

			if (TObjectPtr<UMaterialInterface> RandomMaterial = GEngine->RandomColorMaterial)
			{
				LevelColorationMaterialInstance = new FColoredMaterialRenderProxy(
					RandomMaterial->GetRenderProxy(), 
					GetSelectionColor(RandomColor, bSelected, PrimitiveSceneProxy->IsHovered(), /*bUseOverlayIntensity=*/true), 
					TEXT("Diffuse"));
			}
			else
			{
				const UMaterial* LevelColorationMaterial = EngineShowFlags.Lighting ? GEngine->LevelColorationLitMaterial : GEngine->LevelColorationUnlitMaterial;
				
				// Draw the mesh with level coloration.
				LevelColorationMaterialInstance = new FColoredMaterialRenderProxy(
					LevelColorationMaterial->GetRenderProxy(), 
					GetSelectionColor(RandomColor, bSelected, PrimitiveSceneProxy->IsHovered(), /*bUseOverlayIntensity=*/true));
			}

			Mesh.MaterialRenderProxy = LevelColorationMaterialInstance;
			Collector.RegisterOneFrameMaterialProxy(LevelColorationMaterialInstance);
		}
		else if (EngineShowFlags.Clay)
		{
			if (TObjectPtr<UMaterialInterface> ClayMaterial = GEngine->ClayMaterial)
			{
				FMaterialRenderProxy* LevelColorationMaterialInstance = new FColoredMaterialRenderProxy(
					ClayMaterial->GetRenderProxy(), 
					GetSelectionColor(FLinearColor(1, 1, 1, 1), bSelected, PrimitiveSceneProxy->IsHovered(), /*bUseOverlayIntensity=*/true));

				Mesh.MaterialRenderProxy = LevelColorationMaterialInstance;
				Collector.RegisterOneFrameMaterialProxy(LevelColorationMaterialInstance);
			}
		}
		else if (EngineShowFlags.Zebra)
		{
			if (TObjectPtr<UMaterialInstanceDynamic> ZebraMaterial = GEngine->ZebraMaterialInstance)
			{
				FMaterialRenderProxy* LevelColorationMaterialInstance = new FColoredMaterialRenderProxy(
					ZebraMaterial->GetRenderProxy(), 
					GetSelectionColor(FLinearColor(0, 0, 0, 0), bSelected, PrimitiveSceneProxy->IsHovered(), /*bUseOverlayIntensity=*/true));

				Mesh.MaterialRenderProxy = LevelColorationMaterialInstance;
				Collector.RegisterOneFrameMaterialProxy(LevelColorationMaterialInstance);
			}
		}
		else if (EngineShowFlags.FrontBackFace)
		{
			if (TObjectPtr<UMaterialInterface> FrontBackFaceMaterial = GEngine->FrontBackFaceMaterial)
			{
				FMaterialRenderProxy* LevelColorationMaterialInstance = new FColoredMaterialRenderProxy(
					FrontBackFaceMaterial->GetRenderProxy(),
					GetSelectionColor(FLinearColor(0, 0, 0, 0), bSelected, PrimitiveSceneProxy->IsHovered(), /*bUseOverlayIntensity=*/false));

				Mesh.MaterialRenderProxy = LevelColorationMaterialInstance;
				Collector.RegisterOneFrameMaterialProxy(LevelColorationMaterialInstance);
			}
		}
#endif // WITH_EDITOR
		else if (EngineShowFlags.BSPSplit
			&& PrimitiveSceneProxy->ShowInBSPSplitViewmode())
		{
			// Determine unique color for model component.
			FLinearColor BSPSplitColor;
			FRandomStream RandomStream(GetTypeHash(PrimitiveSceneProxy->GetPrimitiveComponentId().PrimIDValue));
			BSPSplitColor.R = RandomStream.GetFraction();
			BSPSplitColor.G = RandomStream.GetFraction();
			BSPSplitColor.B = RandomStream.GetFraction();
			BSPSplitColor.A = 1.0f;

			// Piggy back on the level coloration material.
			const UMaterial* BSPSplitMaterial = EngineShowFlags.Lighting ? GEngine->LevelColorationLitMaterial : GEngine->LevelColorationUnlitMaterial;
			
			// Draw BSP mesh with unique color for each model component.
			auto BSPSplitMaterialInstance = new FColoredMaterialRenderProxy(
				BSPSplitMaterial->GetRenderProxy(),
				GetSelectionColor(BSPSplitColor,bSelected,PrimitiveSceneProxy->IsHovered())
				);
			Mesh.MaterialRenderProxy = BSPSplitMaterialInstance;
			Collector.RegisterOneFrameMaterialProxy(BSPSplitMaterialInstance);
		}
		else if (PrimitiveSceneProxy->HasStaticLighting() && !PrimitiveSceneProxy->HasValidSettingsForStaticLighting())
		{
			auto InvalidSettingsMaterialInstance = new FColoredMaterialRenderProxy(
				GEngine->InvalidLightmapSettingsMaterial->GetRenderProxy(),
				GetSelectionColor(PrimitiveSceneProxy->GetPrimitiveColor(),bSelected,PrimitiveSceneProxy->IsHovered())
				);
			Mesh.MaterialRenderProxy = InvalidSettingsMaterialInstance;
			Collector.RegisterOneFrameMaterialProxy(InvalidSettingsMaterialInstance);
		}
	}
#endif
}


bool IsUVOutOfBounds(FVector2D UV)
{
	const float FudgeFactor = 1.0f/1024.0f;
	return (UV.X < -FudgeFactor || UV.X > (1.0f+FudgeFactor))
		|| (UV.Y < -FudgeFactor || UV.Y > (1.0f+FudgeFactor));
}

template<class VertexBufferType, class IndexBufferType>
void DrawUVsInternal(FViewport* InViewport, FCanvas* InCanvas, int32 InTextYPos, const int32 LODLevel, int32 UVChannel, TArray<FVector2D> SelectedEdgeTexCoords, VertexBufferType& VertexBuffer, IndexBufferType& Indices )
{
	//draw a string showing what UV channel and LOD is being displayed
	InCanvas->DrawShadowedString( 
		6,
		InTextYPos,
		*FText::Format( NSLOCTEXT("UnrealEd", "UVOverlay_F", "Showing UV channel {0} for LOD {1}"), FText::AsNumber(UVChannel), FText::AsNumber(LODLevel) ).ToString(),
		GEngine->GetSmallFont(),
		FLinearColor::White
		);
	InTextYPos += 18;

	if( ( ( uint32 )UVChannel < VertexBuffer.GetNumTexCoords() ) )
	{
		//calculate scaling
		const uint32 BorderWidth = 5;
		const uint32 MinY = InTextYPos + BorderWidth;
		const uint32 MinX = BorderWidth;
		const FVector2D UVBoxOrigin(MinX, MinY);
		const FVector2D BoxOrigin( MinX - 1, MinY - 1 );
		const uint32 UVBoxScale = FMath::Min(InViewport->GetSizeXY().X / InCanvas->GetDPIScale() - MinX, InViewport->GetSizeXY().Y / InCanvas->GetDPIScale() - MinY) - BorderWidth;
		const uint32 BoxSize = UVBoxScale + 2;
		FCanvasTileItem BoxBackgroundTileItem(BoxOrigin, GWhiteTexture, FVector2D(BoxSize, BoxSize), FLinearColor(0, 0, 0, 0.4f));
		BoxBackgroundTileItem.BlendMode = SE_BLEND_AlphaComposite;
		InCanvas->DrawItem(BoxBackgroundTileItem);
		FCanvasBoxItem BoxItem( BoxOrigin, FVector2D( BoxSize, BoxSize ) );
		BoxItem.SetColor( FLinearColor::Black );
		InCanvas->DrawItem( BoxItem );

		{
			//draw triangles
			uint32 NumIndices = Indices.Num();
			FCanvasLineItem LineItem;
			for (uint32 i = 0; i < NumIndices - 2; i += 3)
			{
				FVector2D UVs[3];
				bool bOutOfBounds[3];

				for (int32 Corner = 0; Corner < 3; Corner++)
				{
					UVs[Corner] = FVector2D(VertexBuffer.GetVertexUV(Indices[i + Corner], UVChannel));
					bOutOfBounds[Corner] = IsUVOutOfBounds(UVs[Corner]);
				}

				for (int32 Edge = 0; Edge < 3; Edge++)
				{
					int32 Corner1 = Edge;
					int32 Corner2 = (Edge + 1) % 3;
					FLinearColor Color = (bOutOfBounds[Corner1] || bOutOfBounds[Corner2]) ? FLinearColor(0.6f, 0.0f, 0.0f) : (SelectedEdgeTexCoords.Num() > 0 ? FLinearColor(0.4f, 0.4f, 0.4f) : FLinearColor::White);
					LineItem.SetColor(Color);
					LineItem.Draw(InCanvas, UVs[Corner1] * UVBoxScale + UVBoxOrigin, UVs[Corner2] * UVBoxScale + UVBoxOrigin);
				}
			}
		}

		{
			// Draw any edges that are currently selected by the user
			FCanvasLineItem LineItem;
			if (SelectedEdgeTexCoords.Num() > 0)
			{
				LineItem.SetColor(FLinearColor::Yellow);
				LineItem.LineThickness = 2.0f;
				for (int32 UVIndex = 0; UVIndex < SelectedEdgeTexCoords.Num(); UVIndex += 2)
				{
					FVector2D UVs[2];
					UVs[0] = (SelectedEdgeTexCoords[UVIndex]);
					UVs[1] = (SelectedEdgeTexCoords[UVIndex + 1]);

					LineItem.Draw(InCanvas, UVs[0] * UVBoxScale + UVBoxOrigin, UVs[1] * UVBoxScale + UVBoxOrigin);
				}
			}
		}
	}
}

void DrawUVs(FViewport* InViewport, FCanvas* InCanvas, int32 InTextYPos, const int32 LODLevel, int32 UVChannel, TArray<FVector2D> SelectedEdgeTexCoords, FStaticMeshRenderData* StaticMeshRenderData, FSkeletalMeshLODRenderData* SkeletalMeshRenderData )
{
	if(StaticMeshRenderData)
	{
		FIndexArrayView IndexBuffer = StaticMeshRenderData->LODResources[LODLevel].IndexBuffer.GetArrayView();
		DrawUVsInternal(InViewport, InCanvas, InTextYPos, LODLevel, UVChannel, SelectedEdgeTexCoords, StaticMeshRenderData->LODResources[LODLevel].VertexBuffers.StaticMeshVertexBuffer, IndexBuffer);
	}
	else if(SkeletalMeshRenderData)
	{
		TArray<uint32> IndexBuffer;
		SkeletalMeshRenderData->MultiSizeIndexContainer.GetIndexBuffer(IndexBuffer);
		DrawUVsInternal(InViewport, InCanvas, InTextYPos, LODLevel, UVChannel, SelectedEdgeTexCoords, SkeletalMeshRenderData->StaticVertexBuffers.StaticMeshVertexBuffer, IndexBuffer);
	}
	else
	{
		check(false); // Must supply either StaticMeshRenderData or SkeletalMeshRenderData
	}
}
