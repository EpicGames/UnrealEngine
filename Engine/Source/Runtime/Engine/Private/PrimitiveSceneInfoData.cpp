// Copyright Epic Games, Inc. All Rights Reserved.

#include "PrimitiveSceneInfoData.h"
#include "GameFramework/Actor.h"

/** Next id to be used by a component. */
// 0 is reserved to mean invalid
FThreadSafeCounter FPrimitiveSceneInfoData::NextPrimitiveId;

void FPrimitiveSceneInfoData::SetLastRenderTime(float InLastRenderTime, bool bUpdateLastRenderTimeOnScreen) const
{
	SetLastRenderTime(InLastRenderTime);

	if (bUpdateLastRenderTimeOnScreen)
	{
		SetLastRenderTimeOnScreen(InLastRenderTime);
	}

	if (OwnerLastRenderTimePtr)
	{
		OwnerLastRenderTimePtr->SetLastRenderTime(InLastRenderTime);
	}
}