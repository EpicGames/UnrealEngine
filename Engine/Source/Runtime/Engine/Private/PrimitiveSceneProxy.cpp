// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	PrimitiveSceneProxy.cpp: Primitive scene proxy implementation.
=============================================================================*/

#include "PrimitiveSceneProxy.h"
#include "PrimitiveSceneProxyDesc.h"
#include "PrimitiveViewRelevance.h"
#include "SceneView.h"
#include "UObject/Package.h"
#include "LevelUtils.h"
#include "Engine/Engine.h"
#include "Engine/LevelStreaming.h"
#include "EngineUtils.h"
#include "Components/BrushComponent.h"
#include "PrimitiveSceneInfo.h"
#include "Materials/Material.h"
#include "Materials/MaterialRenderProxy.h"
#include "MaterialDomain.h"
#include "MaterialShared.h"
#include "RenderUtils.h"
#include "VT/RuntimeVirtualTexture.h"
#include "NaniteSceneProxy.h" // TODO: PROG_RASTER
#include "ComponentRecreateRenderStateContext.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "PrimitiveUniformShaderParametersBuilder.h"
#include "InstanceDataSceneProxy.h"
#include "GameFramework/ActorPrimitiveColorHandler.h"
#include "Streaming/SimpleStreamableAssetManager.h"

#if WITH_EDITOR
#include "ObjectCacheEventSink.h"
#endif

static TAutoConsoleVariable<int32> CVarForceSingleSampleShadowingFromStationary(
	TEXT("r.Shadow.ForceSingleSampleShadowingFromStationary"),
	0,
	TEXT("Whether to force all components to act as if they have bSingleSampleShadowFromStationaryLights enabled.  Useful for scalability when dynamic shadows are disabled."),
	ECVF_RenderThreadSafe | ECVF_Scalability
	);

static TAutoConsoleVariable<bool> CVarOptimizedWPOAffectNonNaniteShaderSelection(
	TEXT("r.OptimizedWPO.AffectNonNaniteShaderSelection"),
	false,
	TEXT("Whether the per primitive WPO flag should affect shader selection for non-nanite primitives. It increase the chance of selecting the position only depth VS ")
	TEXT("at the cost of updating cached draw commands whenever the WPO flag changes."),
	FConsoleVariableDelegate::CreateLambda([](IConsoleVariable* InVariable)
	{
		FGlobalComponentRecreateRenderStateContext Context;
	}),
	ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<int32> CVarCacheWPOPrimitives(
	TEXT("r.Shadow.CacheWPOPrimitives"),
	0,
	TEXT("Whether primitives whose materials use World Position Offset should be considered movable for cached shadowmaps.\n")
	TEXT("Enablings this gives more correct, but slower whole scene shadows from materials that use WPO."),
	ECVF_RenderThreadSafe | ECVF_Scalability
	);

static TAutoConsoleVariable<int32> CVarVelocityEnableVertexDeformation(
	TEXT("r.Velocity.EnableVertexDeformation"),
	2,
	TEXT("Enables materials with World Position Offset and/or World Displacement to output velocities during velocity pass even when the actor has not moved. \n")
	TEXT("0=Off, 1=On, 2=Auto(Default). \n")
	TEXT("Auto setting is off if r.VelocityOutputPass=2, or else on. \n")
	TEXT("When r.VelocityOutputPass=2 this can incur a performance cost due to additional draw calls."),
	FConsoleVariableDelegate::CreateLambda([](IConsoleVariable* InVariable)
	{
		FGlobalComponentRecreateRenderStateContext Context;
	}),
	ECVF_RenderThreadSafe
	);

static TAutoConsoleVariable<int32> CVarVelocityForceOutput(
	TEXT("r.Velocity.ForceOutput"), 0,
	TEXT("Force velocity output on all primitives.\n")
	TEXT("This can incur a performance cost unless r.VelocityOutputPass=1.\n")
	TEXT("But it can be useful for testing where velocity output isn't being enabled as expected.\n")
	TEXT("0: Disabled (default)\n")
	TEXT("1: Enabled"),
	FConsoleVariableDelegate::CreateLambda([](IConsoleVariable* InVariable)
	{
		FGlobalComponentRecreateRenderStateContext Context;
	}),
	ECVF_RenderThreadSafe
	);

static TAutoConsoleVariable<int32> CVarApproximateOcclusionQueries(
	TEXT("r.ApproximateOcclusionQueries"),
	0,
	TEXT("Batch occlusion for a static and skeletal mesh even if there are movable.In general it's more beneficial to batch occlusion queires for all meshes"),
	ECVF_RenderThreadSafe
);

bool GParallelGatherDynamicMeshElements = true;
static FAutoConsoleVariableRef CVarParallelGatherDynamicMeshElements(
	TEXT("r.Visibility.DynamicMeshElements.Parallel"),
	GParallelGatherDynamicMeshElements,
	TEXT("Enables parallel processing of the gather dynamic mesh elements visibility phase."),
	ECVF_RenderThreadSafe
);

bool IsParallelGatherDynamicMeshElementsEnabled()
{
	return GParallelGatherDynamicMeshElements &&
		// parallel GDME creates RHI resources on a task threads which is not supported by some RHIs (eg. OpenGL)
		GSupportsParallelRenderingTasksWithSeparateRHIThread;
}

bool GParallelGatherDynamicRayTracingInstances = true;
static FAutoConsoleVariableRef CVarParallelGatherDynamicRayTracingInstances(
	TEXT("r.RayTracing.DynamicInstances.Parallel"),
	GParallelGatherDynamicRayTracingInstances,
	TEXT("Enables parallel processing of the gather dynamic ray tracing instances."),
	ECVF_RenderThreadSafe
);

bool GEnableObjectWorldPositionFix = true;
static FAutoConsoleVariableRef CVarEnableObjectWorldPositionFix(
	TEXT("r.EnableObjectWorldPositionFix"),
	GEnableObjectWorldPositionFix,
	TEXT("If enabled (default) any primitives where HasInstanceDataBuffers() is false will use the previous behavior for calculating ObjectWorldPosition."),
	FConsoleVariableDelegate::CreateLambda([](IConsoleVariable* InVariable)
	{
		FGlobalComponentRecreateRenderStateContext Context;
	}),
	ECVF_RenderThreadSafe
);

bool IsParallelGatherDynamicRayTracingInstancesEnabled()
{
	return GParallelGatherDynamicRayTracingInstances &&
		// parallel GDRTI creates RHI resources on a task threads which is not supported by some RHIs (eg. OpenGL)
		GSupportsParallelRenderingTasksWithSeparateRHIThread;
}

bool FPrimitiveSceneProxy::ShouldRenderCustomDepth() const
{
	return IsCustomDepthPassEnabled() && bRenderCustomDepth;
}

bool IsAllowingApproximateOcclusionQueries()
{
	return CVarApproximateOcclusionQueries.GetValueOnAnyThread() != 0;
}

bool ShouldOptimizedWPOAffectNonNaniteShaderSelection()
{
	return CVarOptimizedWPOAffectNonNaniteShaderSelection.GetValueOnAnyThread() != 0;
}

bool CacheShadowDepthsFromPrimitivesUsingWPO()
{
	return CVarCacheWPOPrimitives.GetValueOnAnyThread() != 0;
}

bool SupportsCachingMeshDrawCommands(const FMeshBatch& MeshBatch)
{
	return
		// Cached mesh commands only allow for a single mesh element per batch.
		(MeshBatch.Elements.Num() == 1) &&

		// View dependent arguments can't be cached
		(MeshBatch.bViewDependentArguments == 0) &&

		// Vertex factory needs to support caching.
		MeshBatch.VertexFactory->GetType()->SupportsCachingMeshDrawCommands();
}

bool SupportsCachingMeshDrawCommands(const FMeshBatch& MeshBatch, ERHIFeatureLevel::Type FeatureLevel)
{
	if (SupportsCachingMeshDrawCommands(MeshBatch))
	{
		// External textures get mapped to immutable samplers (which are part of the PSO); the mesh must go through the dynamic path, as the media player might not have
		// valid textures/samplers the first few calls; once they're available the PSO needs to get invalidated and recreated with the immutable samplers.
		const FMaterial& Material = MeshBatch.MaterialRenderProxy->GetIncompleteMaterialWithFallback(FeatureLevel);
		const FMaterialShaderMap* ShaderMap = Material.GetRenderingThreadShaderMap();
		if (ShaderMap)
		{
			const FUniformExpressionSet& ExpressionSet = ShaderMap->GetUniformExpressionSet();
			if (ExpressionSet.HasExternalTextureExpressions())
			{
				return false;
			}
		}

		return true;
	}

	return false;
}

bool SupportsNaniteRendering(const FVertexFactory* RESTRICT VertexFactory, const FPrimitiveSceneProxy* RESTRICT PrimitiveSceneProxy)
{
	// Volumetric self shadow mesh commands need to be generated every frame, as they depend on single frame uniform buffers with self shadow data.
	return VertexFactory->GetType()->SupportsNaniteRendering();
}

bool SupportsNaniteRendering(const FVertexFactory* RESTRICT VertexFactory, const FPrimitiveSceneProxy* RESTRICT PrimitiveSceneProxy, const FMaterialRenderProxy* MaterialRenderProxy, ERHIFeatureLevel::Type FeatureLevel)
{
	if (FeatureLevel >= ERHIFeatureLevel::SM5 && SupportsNaniteRendering(VertexFactory, PrimitiveSceneProxy))
	{
		const FMaterial& Material = MaterialRenderProxy->GetIncompleteMaterialWithFallback(FeatureLevel);
		const FMaterialShaderMap* ShaderMap = Material.GetRenderingThreadShaderMap();

		return (Material.IsUsedWithNanite() || Material.IsSpecialEngineMaterial()) &&
			Nanite::IsSupportedBlendMode(Material) &&
			Nanite::IsSupportedMaterialDomain(Material.GetMaterialDomain());
	}

	return false;
}


bool SupportsNaniteRendering(const FVertexFactoryType* RESTRICT VertexFactoryType, const class FMaterial& Material, ERHIFeatureLevel::Type FeatureLevel)
{
	if (FeatureLevel >= ERHIFeatureLevel::SM5 && VertexFactoryType->SupportsNaniteRendering())
	{
		const FMaterialShaderMap* ShaderMap = Material.GetGameThreadShaderMap();
		return (Material.IsUsedWithNanite() || Material.IsSpecialEngineMaterial()) &&
			Nanite::IsSupportedBlendMode(Material) &&
			Nanite::IsSupportedMaterialDomain(Material.GetMaterialDomain());
	}

	return false;
}


static bool VertexDeformationOutputsVelocity()
{
	static const auto CVarVelocityOutputPass = IConsoleManager::Get().FindConsoleVariable(TEXT("r.VelocityOutputPass"));
	const bool bVertexDeformationOutputsVelocityDefault = CVarVelocityOutputPass && CVarVelocityOutputPass->GetInt() != 2;
	const int32 VertexDeformationOutputsVelocity = CVarVelocityEnableVertexDeformation.GetValueOnAnyThread();
	return VertexDeformationOutputsVelocity == 1 || (VertexDeformationOutputsVelocity == 2 && bVertexDeformationOutputsVelocityDefault);
}

static FBoxSphereBounds PadBounds(const FBoxSphereBounds& InBounds, float PadAmount)
{
	FBoxSphereBounds Result = InBounds;
	Result.BoxExtent += FVector(PadAmount);
	Result.SphereRadius += PadAmount * UE_SQRT_3;

	return Result;
}

static FVector GetLocalBoundsPadExtent(const FMatrix& LocalToWorld, float PadAmount)
{
	if (FMath::Abs(PadAmount) < UE_SMALL_NUMBER)
	{
		return FVector::ZeroVector;
	}

	FVector InvScale = LocalToWorld.GetScaleVector();
	InvScale = FVector(
		InvScale.X > 0.0 ? 1.0 / InvScale.X : 0.0,
		InvScale.Y > 0.0 ? 1.0 / InvScale.Y : 0.0,
		InvScale.Z > 0.0 ? 1.0 / InvScale.Z : 0.0);

	return FVector(PadAmount) * InvScale;
}

static FBoxSphereBounds PadLocalBounds(const FBoxSphereBounds& InBounds, const FMatrix& LocalToWorld, float PadAmount)
{
	const FVector PadExtent = GetLocalBoundsPadExtent(LocalToWorld, PadAmount);

	FBoxSphereBounds Result = InBounds;
	Result.BoxExtent += PadExtent;
	Result.SphereRadius += PadExtent.Length();

	return Result;
}

static FRenderBounds PadLocalRenderBounds(const FRenderBounds& InBounds, const FMatrix& LocalToWorld, float PadAmount)
{
	const FVector3f PadExtent = FVector3f(GetLocalBoundsPadExtent(LocalToWorld, PadAmount));

	FRenderBounds Result = InBounds;
	Result.Min -= PadExtent;
	Result.Max += PadExtent;

	return Result;
}

FPrimitiveSceneProxyDesc::FPrimitiveSceneProxyDesc(const UPrimitiveComponent* InComponent)
	: FPrimitiveSceneProxyDesc()
{
	InitializeFromPrimitiveComponent(InComponent);
}

// NOTE: required because of: bCanSkipRedundantTransformUpdates, remove when cleaned up
PRAGMA_DISABLE_DEPRECATION_WARNINGS
FPrimitiveSceneProxy::FPrimitiveSceneProxy() = default;
FPrimitiveSceneProxy::FPrimitiveSceneProxy(FPrimitiveSceneProxy const&) = default;
FPrimitiveSceneProxy::FPrimitiveSceneProxy(FPrimitiveSceneProxy&&) = default;
FPrimitiveSceneProxy& FPrimitiveSceneProxy::operator=(const FPrimitiveSceneProxy&) = default;
FPrimitiveSceneProxy& FPrimitiveSceneProxy::operator=(FPrimitiveSceneProxy&&) = default;
PRAGMA_ENABLE_DEPRECATION_WARNINGS

FPrimitiveSceneProxy::FPrimitiveSceneProxy(const UPrimitiveComponent* InComponent, FName InResourceName)
	: FPrimitiveSceneProxy(FPrimitiveSceneProxyDesc(InComponent), InResourceName)
{
}

FPrimitiveSceneProxy::FPrimitiveSceneProxy(const FPrimitiveSceneProxyDesc& InProxyDesc, FName InResourceName) :
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	WireframeColor(FLinearColor::White)
,	
#endif
	CustomPrimitiveData(InProxyDesc.GetCustomPrimitiveData())
,	TranslucencySortPriority(FMath::Clamp(InProxyDesc.TranslucencySortPriority, SHRT_MIN, SHRT_MAX))
,	TranslucencySortDistanceOffset(InProxyDesc.TranslucencySortDistanceOffset)
,	Mobility(InProxyDesc.Mobility)
,	LightmapType(InProxyDesc.LightmapType)
,	StatId()
,	DrawInGame(InProxyDesc.IsVisible())
,	DrawInEditor(InProxyDesc.IsVisibleEditor())
,	bReceivesDecals(InProxyDesc.bReceivesDecals)
,	bVirtualTextureMainPassDrawAlways(true)
,	bVirtualTextureMainPassDrawNever(false)
,	bOnlyOwnerSee(InProxyDesc.bOnlyOwnerSee)
,	bOwnerNoSee(InProxyDesc.bOwnerNoSee)
,	bParentSelected(InProxyDesc.ShouldRenderSelected())
,	bIndividuallySelected(InProxyDesc.IsComponentIndividuallySelected())
,	bLevelInstanceEditingState(InProxyDesc.GetLevelInstanceEditingState())
,	bHovered(false)
,	bUseViewOwnerDepthPriorityGroup(InProxyDesc.bUseViewOwnerDepthPriorityGroup)
,	StaticDepthPriorityGroup((uint8)InProxyDesc.GetStaticDepthPriorityGroup())
,	ViewOwnerDepthPriorityGroup(InProxyDesc.ViewOwnerDepthPriorityGroup)
,	bStaticLighting(InProxyDesc.HasStaticLighting())
,	bVisibleInReflectionCaptures(InProxyDesc.bVisibleInReflectionCaptures)
,	bVisibleInRealTimeSkyCaptures(InProxyDesc.bVisibleInRealTimeSkyCaptures)
,	bVisibleInRayTracing(InProxyDesc.bVisibleInRayTracing)
,	bVisibleInReflections(InProxyDesc.bVisibleInReflections)
,	bRenderInDepthPass(InProxyDesc.bRenderInDepthPass)
,	bRenderInMainPass(InProxyDesc.bRenderInMainPass)
,	bForceHidden(false)
,	bCollisionEnabled(InProxyDesc.IsCollisionEnabled())
,	bTreatAsBackgroundForOcclusion(InProxyDesc.bTreatAsBackgroundForOcclusion)
,	bSupportsParallelGDME(true)
,	bSinglePassGDME(false)
,	bSupportsParallelGDRTI(true)
,	bVisibleInLumenScene(false)
,	bOpaqueOrMasked(true)
PRAGMA_DISABLE_DEPRECATION_WARNINGS
,	bCanSkipRedundantTransformUpdates(true)
PRAGMA_ENABLE_DEPRECATION_WARNINGS
,	bGoodCandidateForCachedShadowmap(true)
,	bNeedsUnbuiltPreviewLighting(!InProxyDesc.IsPrecomputedLightingValid())
,	bHasValidSettingsForStaticLighting(InProxyDesc.HasValidSettingsForStaticLighting())
,	bWillEverBeLit(true)
	// Disable dynamic shadow casting if the primitive only casts indirect shadows, since dynamic shadows are always shadowing direct lighting
,	bCastDynamicShadow(InProxyDesc.bCastDynamicShadow && InProxyDesc.CastShadow && !InProxyDesc.GetShadowIndirectOnly())
,	bEmissiveLightSource(InProxyDesc.bEmissiveLightSource)
,   bAffectDynamicIndirectLighting(InProxyDesc.bAffectDynamicIndirectLighting)
,	bAffectIndirectLightingWhileHidden(InProxyDesc.bAffectDynamicIndirectLighting && InProxyDesc.bAffectIndirectLightingWhileHidden)
,   bAffectDistanceFieldLighting(InProxyDesc.bAffectDistanceFieldLighting)
,   bSupportsMaterialCache(false)
,	bCastStaticShadow(InProxyDesc.CastShadow && InProxyDesc.bCastStaticShadow)
,	ShadowCacheInvalidationBehavior(InProxyDesc.ShadowCacheInvalidationBehavior)
,	bCastVolumetricTranslucentShadow(InProxyDesc.bCastDynamicShadow && InProxyDesc.CastShadow && InProxyDesc.bCastVolumetricTranslucentShadow)
,	bCastContactShadow(InProxyDesc.CastShadow && InProxyDesc.bCastContactShadow)
,	bCastDeepShadow(false)
,	bCastCapsuleDirectShadow(false)
,	bCastsDynamicIndirectShadow(false)
,	bCastHiddenShadow(InProxyDesc.CastShadow && InProxyDesc.bCastHiddenShadow)
,	bCastShadowAsTwoSided(InProxyDesc.bCastShadowAsTwoSided)
,	bSelfShadowOnly(InProxyDesc.bSelfShadowOnly)
,	bCastInsetShadow(InProxyDesc.bSelfShadowOnly ? true : InProxyDesc.bCastInsetShadow)	// Assumed to be enabled if bSelfShadowOnly is enabled.
,	bCastCinematicShadow(InProxyDesc.bCastCinematicShadow)
,	bCastFarShadow(InProxyDesc.bCastFarShadow)
,	bLightAttachmentsAsGroup(InProxyDesc.bLightAttachmentsAsGroup)
,	bSingleSampleShadowFromStationaryLights(InProxyDesc.bSingleSampleShadowFromStationaryLights)
,	bStaticElementsAlwaysUseProxyPrimitiveUniformBuffer(false)
,	bVFRequiresPrimitiveUniformBuffer(true)
,	bDoesMeshBatchesUseSceneInstanceCount(false)
,	bIsStaticMesh(false)
,	bIsNaniteMesh(false)
,	bIsAlwaysVisible(false)
,	bIsHeterogeneousVolume(false)
,	bIsHierarchicalInstancedStaticMesh(false)
,	bIsLandscapeGrass(false)
,	bSupportsGPUScene(false)
,	bHasDeformableMesh(true)
,	bEvaluateWorldPositionOffset(true)
,	bHasWorldPositionOffsetVelocity(false)
,	bAnyMaterialHasWorldPositionOffset(false)
,	bAnyMaterialAlwaysEvaluatesWorldPositionOffset(false)
,	bAnyMaterialHasPixelAnimation(false)
,   bAnyMaterialUsesTemporalResponsiveness(false)
,   bAnyMaterialUsesMotionVectorWorldOffset(false)
,	bSupportsDistanceFieldRepresentation(false)
,	bSupportsHeightfieldRepresentation(false)
,	bLumenHeightfield(InProxyDesc.bLumenHeightfield)
,   bIsLandscapeProxy(false)
,	bIsLandscapeNaniteProxy(false)
,	bCompatibleWithLumenCardSharing(false)
,	bSupportsSortedTriangles(false)
,	bShouldNotifyOnWorldAddRemove(false)
,	bWantsSelectionOutline(true)
,	bVerifyUsedMaterials(true)
,   bWantsEditorEffects(InProxyDesc.bWantsEditorEffects)
,	bAllowApproximateOcclusion(InProxyDesc.Mobility != EComponentMobility::Movable)
,   bHoldout(InProxyDesc.bHoldout)
,	bSplineMesh(false)
,	bSkinnedMesh(false)
,	bInstancedSkinnedMesh(false)
,	bIsFirstPerson(InProxyDesc.bIsFirstPerson)
,	bIsFirstPersonWorldSpaceRepresentation(InProxyDesc.bIsFirstPersonWorldSpaceRepresentation)
,	bSupportsRuntimeVirtualTexture(false)
,	bImplementsStreamableAssetGathering(false)
,	bUseAsOccluder(InProxyDesc.bUseAsOccluder)
,	bSelectable(InProxyDesc.bSelectable)
,	bHasPerInstanceHitProxies(InProxyDesc.bHasPerInstanceHitProxies)
,	bUseEditorCompositing(InProxyDesc.bUseEditorCompositing)
,	bIsBeingMovedByEditor(InProxyDesc.bIsBeingMovedByEditor)
,	bReceiveMobileCSMShadows(InProxyDesc.bReceiveMobileCSMShadows)
,	bRenderCustomDepth(InProxyDesc.bRenderCustomDepth)
,	bVisibleInSceneCaptureOnly(InProxyDesc.bVisibleInSceneCaptureOnly)
,	bHiddenInSceneCapture(InProxyDesc.bHiddenInSceneCapture)
,	bForceMipStreaming(InProxyDesc.bForceMipStreaming)
,	bRayTracingFarField(InProxyDesc.bRayTracingFarField)
,	CustomDepthStencilValue(InProxyDesc.CustomDepthStencilValue)
,	CustomDepthStencilWriteMask(FRendererStencilMaskEvaluation::ToStencilMask(InProxyDesc.CustomDepthStencilWriteMask))
,	LightingChannelMask(GetLightingChannelMaskForStruct(InProxyDesc.LightingChannels))
,	RayTracingGroupId(InProxyDesc.GetRayTracingGroupId())
,	RayTracingGroupCullingPriority((uint8)InProxyDesc.RayTracingGroupCullingPriority)
,	IndirectLightingCacheQuality(InProxyDesc.IndirectLightingCacheQuality)
,	VirtualTextureLodBias(InProxyDesc.VirtualTextureLodBias)
,	VirtualTextureCullMips(InProxyDesc.VirtualTextureCullMips)
,	VirtualTextureMinCoverage(InProxyDesc.VirtualTextureMinCoverage)
,	DynamicIndirectShadowMinVisibility(0)
,	DistanceFieldSelfShadowBias(0.0f)
,	MaxWPOExtent(0.0f)
,	MinMaxMaterialDisplacement(0.0f, 0.0f)
,	PrimitiveComponentId(InProxyDesc.ComponentId)
,	Scene(InProxyDesc.GetScene())
,	PrimitiveSceneInfo(nullptr)
,	OwnerName(InProxyDesc.GetOwner() ? InProxyDesc.GetOwner()->GetFName() : NAME_None)
,	ResourceName(InResourceName)
,	LevelName(InProxyDesc.GetLevel() ? InProxyDesc.GetLevel()->GetOutermost()->GetFName() : NAME_None)
#if WITH_EDITOR
// by default we are always drawn
,	HiddenEditorViews(0)
,   OverlayColor(InProxyDesc.OverlayColor)
,   SelectionOutlineColorIndex(0)
,	DrawInAnyEditMode(0)
,   bIsFoliage(false)
#endif
,	VisibilityId(InProxyDesc.VisibilityId)
, 	ComponentForDebuggingOnly(Cast<UPrimitiveComponent>(InProxyDesc.Component))
#if WITH_EDITOR
,	NumUncachedStaticLightingInteractions(0)
#endif
,	MaxDrawDistance(InProxyDesc.CachedMaxDrawDistance > 0 ? InProxyDesc.CachedMaxDrawDistance : FLT_MAX)
,	MinDrawDistance(InProxyDesc.MinDrawDistance)
{
	check(Scene);

	// Initialize ForceHidden flag based on Level's visibility (only if Level bRequireFullVisibilityToRender is set)
	if (ULevel* Level = InProxyDesc.GetLevel())
	{
		bShouldNotifyOnWorldAddRemove = Level->bRequireFullVisibilityToRender;
		if (bShouldNotifyOnWorldAddRemove)
		{
			SetForceHidden(!Level->bIsVisible);
		}

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
		if (UPrimitiveComponent* PrimitiveComponent = Cast<UPrimitiveComponent>(InProxyDesc.Component))
		{
			SetPrimitiveColor(FActorPrimitiveColorHandler::Get().GetPrimitiveColor(PrimitiveComponent));
		}
		else
		{
			SetPrimitiveColor(FLinearColor::White);
		}
#endif
	}

#if STATS
	{
		if (UObject const* StatObject = InProxyDesc.AdditionalStatObject()) // prefer the additional object, this is usually the thing related to the component)
		{
			StatId = StatObject->GetStatID(true);
		}
		else
		{
			StatId = InProxyDesc.GetStatID(true);
		}
	}
#endif

#if MESH_DRAW_COMMAND_STATS
	MeshDrawCommandStatsCategory = InProxyDesc.GetMeshDrawCommandStatsCategory();
#endif

	if (bNeedsUnbuiltPreviewLighting && !bHasValidSettingsForStaticLighting)
	{
		// Don't use unbuilt preview lighting for static components that have an invalid lightmap UV setup
		// Otherwise they would light differently in editor and in game, even after a lighting rebuild
		bNeedsUnbuiltPreviewLighting = false;
	}
	
	{
		DrawInGame &= !InProxyDesc.IsHidden();
		#if WITH_EDITOR
			DrawInEditor &= !InProxyDesc.IsHiddenEd();
		#endif

		if(bOnlyOwnerSee || bOwnerNoSee || bUseViewOwnerDepthPriorityGroup || bIsFirstPersonWorldSpaceRepresentation)
		{
			Owners = MoveTemp(InProxyDesc.ActorOwners);
		}

#if WITH_EDITOR
		// cache the actor's group membership
		HiddenEditorViews = InProxyDesc.GetHiddenEditorViews();
		DrawInAnyEditMode = InProxyDesc.IsOwnerEditorOnly();
		bIsFoliage = InProxyDesc.IsOwnedByFoliage();
#endif
	}	

	// Setup the runtime virtual texture information
	if (UseVirtualTexturing(GetScene().GetShaderPlatform()))
	{
		for (URuntimeVirtualTexture* RuntimeVirtualTexture : InProxyDesc.GetRuntimeVirtualTextures())
		{
			if (RuntimeVirtualTexture != nullptr)
			{
				RuntimeVirtualTextures.Add(RuntimeVirtualTexture->GetUniqueID());
				RuntimeVirtualTextureMaterialTypes.AddUnique(RuntimeVirtualTexture->GetMaterialType());
			}
		}
	}

	// Conditionally remove from the main passes based on the runtime virtual texture setup
	const bool bRequestVirtualTexture = InProxyDesc.GetRuntimeVirtualTextures().Num() > 0;
	if (bRequestVirtualTexture)
	{
		ERuntimeVirtualTextureMainPassType MainPassType = InProxyDesc.GetVirtualTextureRenderPassType();
		bVirtualTextureMainPassDrawNever = MainPassType == ERuntimeVirtualTextureMainPassType::Never;
		bVirtualTextureMainPassDrawAlways = MainPassType == ERuntimeVirtualTextureMainPassType::Always;
	}

	// Modify max draw distance for main pass if we are using virtual texturing
	const bool bUseVirtualTexture = RuntimeVirtualTextures.Num() > 0;
	if (bUseVirtualTexture && InProxyDesc.GetVirtualTextureMainPassMaxDrawDistance() > 0.f)
	{
		MaxDrawDistance = FMath::Min(MaxDrawDistance, InProxyDesc.GetVirtualTextureMainPassMaxDrawDistance());
	}

#if WITH_EDITOR	
	const bool bGetDebugMaterials = true;
	InProxyDesc.GetUsedMaterials(UsedMaterialsForVerification, bGetDebugMaterials);

	// If InProxyDesc can't provide a PrimitiveComponentInterface we can't be notified about updates
	if (InProxyDesc.GetPrimitiveComponentInterface())
	{
		FObjectCacheEventSink::NotifyUsedMaterialsChanged_Concurrent(InProxyDesc.GetPrimitiveComponentInterface(), UsedMaterialsForVerification);
	}
#endif

	bAnyMaterialHasWorldPositionOffset = false;
	bAnyMaterialHasPixelAnimation = false;
	bAnyMaterialUsesTemporalResponsiveness = false;
	bAnyMaterialUsesMotionVectorWorldOffset = false;
	{
		// Find if we have any WPO materials.
		EShaderPlatform ShaderPlatform = GetScene().GetShaderPlatform();

		TArray<UMaterialInterface*> UsedMaterials;
		InProxyDesc.GetUsedMaterials(UsedMaterials);
		for (const UMaterialInterface* MaterialInterface : UsedMaterials)
		{
			if (MaterialInterface && MaterialInterface->IsUsingWorldPositionOffset_Concurrent(ShaderPlatform))
			{
				bAnyMaterialHasWorldPositionOffset = true;
				break;
			}
		}

		if (VelocityEncodeHasPixelAnimation(GetScene().GetShaderPlatform())
			// Currently, only TSR checks the HasPixelAnimation flag but setting it will force velocity writes even if TSR is not used
			&& SupportsTSR(GetScene().GetShaderPlatform()))
		{
			for (const UMaterialInterface* MaterialInterface : UsedMaterials)
			{
				if (MaterialInterface)
				{
					if (MaterialInterface->HasPixelAnimation() && IsOpaqueOrMaskedBlendMode(MaterialInterface->GetBlendMode()))
					{
						bAnyMaterialHasPixelAnimation = true;
						break;
					}
				}
			}

			for (const UMaterialInterface* MaterialInterface : UsedMaterials)
			{
				if (MaterialInterface && MaterialInterface->IsUsingTemporalResponsiveness_Concurrent(ShaderPlatform))
				{
					bAnyMaterialUsesTemporalResponsiveness = true;
					break;
				}
			}
		}

		for (const UMaterialInterface* MaterialInterface : UsedMaterials)
		{
			if (MaterialInterface && MaterialInterface->IsUsingMotionVectorWorldOffset_Concurrent(ShaderPlatform))
			{
				bAnyMaterialUsesMotionVectorWorldOffset = true;
				break;
			}
		}
	}

	bAlwaysHasVelocity = CVarVelocityForceOutput.GetValueOnAnyThread();
	if (!bAlwaysHasVelocity && InProxyDesc.SupportsWorldPositionOffsetVelocity() && VertexDeformationOutputsVelocity() && bAnyMaterialHasWorldPositionOffset)
	{
		bHasWorldPositionOffsetVelocity = true;
	}

#if UE_WITH_PSO_PRECACHING
	PSOPrecacheRequestsToBoostOnDraw = InProxyDesc.MaterialPSOPrecacheRequestIDsToBoostOnDraw;
#endif

	// FirstPerson primitives won't properly work with most shadow solutions, so just disable them here to give a smoother user experience out of the box.
	// Such primitives are intended to be used in conjunction with FirstPersonWorldSpaceRepresentation primitives, which are used to cast shadow onto the scene.
	// Self shadowing for FirstPerson primitives is solved with First Person Self Shadows, which are currently implemented with screen space ray marching.
	// Note that bCastContactShadow and a few of the legacy shadow cast flags are not overridden here. This is intentional as there might be valid use cases for them in this context.
	if (bIsFirstPerson)
	{
		bCastDynamicShadow = false;
		bCastStaticShadow = false;
		bCastVolumetricTranslucentShadow = false;
		bCastFarShadow = false;
	}
	// FirstPersonWorldSpaceRepresentation primitives must have bOwnerNoSee=true and bCastHiddenShadow=false.
	// This way, they are skipped by regular shadow rendering and can be picked up by the special first person
	// shadow path in VSM. Raytraced shadows ignore these flags anyways, so this works for all cases.
	else if (bIsFirstPersonWorldSpaceRepresentation)
	{
		bOwnerNoSee = true;
		bCastHiddenShadow = false;
	}
}

void FPrimitiveSceneProxy::OnLevelAddedToWorld_RenderThread()
{
	check(bShouldNotifyOnWorldAddRemove);
	SetForceHidden(false);
}

void FPrimitiveSceneProxy::OnLevelRemovedFromWorld_RenderThread()
{
	check(bShouldNotifyOnWorldAddRemove);
	SetForceHidden(true);
}

#if WITH_EDITOR
void FPrimitiveSceneProxy::SetUsedMaterialForVerification(const TArray<UMaterialInterface*>& InUsedMaterialsForVerification)
{
	UsedMaterialsForVerification = InUsedMaterialsForVerification;
}
#endif

FPrimitiveSceneProxy::~FPrimitiveSceneProxy()
{
	// Safety net for proxy-deletion paths that bypass FScene::RemovePrimitive (proxy-only
	// callers without a UPrimitiveComponent, error paths that abort before AddPrimitive
	// completes, custom proxy lifecycles, unit tests). Without this the slot in
	// PrimitiveHandleTable plus any FAssetBoundElement rows accumulated from queued
	// FRegister processing would leak. Idempotent against the component-side recovery in
	// UPrimitiveComponent::BeginDestroy: RecoverMissedUnregister re-checks
	// IsPrimitiveHandleLive, so whichever path fires second is a no-op.
	if (FSimpleStreamableAssetManager::IsEnabled())
	{
		const FSSAMPrimitiveHandle Captured = SSAMPrimitiveHandle.Exchange(FSSAMPrimitiveHandle{});
		if (Captured.IsValid() && !Captured.IsStale())
		{
			FSimpleStreamableAssetManager::RecoverMissedUnregister(Captured);
		}
	}
}

// Potentially invoked by SceneProxy types who still create their proxies through the legacy path
HHitProxy* FPrimitiveSceneProxy::CreateHitProxies(UPrimitiveComponent* Component,TArray<TRefCountPtr<HHitProxy> >& OutHitProxies)
{
	return FPrimitiveSceneProxy::CreateHitProxies(Component->GetPrimitiveComponentInterface(), OutHitProxies);
}
 
HHitProxy* FPrimitiveSceneProxy::CreateHitProxies(IPrimitiveComponent* ComponentInterface,TArray<TRefCountPtr<HHitProxy> >& OutHitProxies)
{
	return ComponentInterface->CreatePrimitiveHitProxies(OutHitProxies);
}

HHitProxy* FActorPrimitiveComponentInterface::CreatePrimitiveHitProxies(TArray<TRefCountPtr<HHitProxy> >& OutHitProxies) 
{
	UPrimitiveComponent* Component = UPrimitiveComponent::GetPrimitiveComponent(this);	

	if(Component->GetOwner())
	{
		HHitProxy* ActorHitProxy;

		if (Component->GetOwner()->IsA(ABrush::StaticClass()) && Component->IsA(UBrushComponent::StaticClass()))
		{
			ActorHitProxy = new HActor(Component->GetOwner(), Component, HPP_Wireframe);
		}
		else
		{
#if WITH_EDITORONLY_DATA
			ActorHitProxy = new HActor(Component->GetOwner(), Component, Component->HitProxyPriority);
#else
			ActorHitProxy = new HActor(Component->GetOwner(), Component);
#endif
		}
		OutHitProxies.Add(ActorHitProxy);
		return ActorHitProxy;
	}
	else
	{
		return NULL;
	}
}

FMatrix FPrimitiveSceneProxy::GetMeshCardsToWorld(int32 InstanceIndex) const
{
	if (const FInstanceSceneDataBuffers* InstanceData = GetInstanceSceneDataBuffers())
	{
		// Instance data must be available on CPU.
		check(!InstanceData->IsInstanceDataGPUOnly());

		const int32 PrimitiveInstanceIndex = FMath::Clamp(InstanceIndex, 0, InstanceData->GetNumInstances() - 1);
		return InstanceData->GetInstanceToWorld(PrimitiveInstanceIndex);
	}
	else
	{
		return GetLocalToWorld();
	}
}

FPrimitiveViewRelevance FPrimitiveSceneProxy::GetViewRelevance(const FSceneView* View) const
{
	return FPrimitiveViewRelevance();
}

void FPrimitiveSceneProxy::CreateUniformBuffer()
{
	if (!UniformBuffer && DoesVFRequirePrimitiveUniformBuffer())
	{
		UniformBuffer = TUniformBufferRef<FPrimitiveUniformShaderParameters>::CreateEmptyUniformBufferImmediate(UniformBuffer_MultiFrame);
	}
}

void FPrimitiveSceneProxy::UpdateUniformBuffer(FRHICommandList& RHICmdList)
{
	// stat disabled by default due to low-value/high-frequency
	//QUICK_SCOPE_CYCLE_COUNTER(STAT_FPrimitiveSceneProxy_UpdateUniformBuffer);

	// Skip expensive primitive uniform buffer creation for proxies whose vertex factories only use GPUScene for primitive data
	if (DoesVFRequirePrimitiveUniformBuffer())
	{
		// Update the uniform shader parameters.
		FPrimitiveUniformShaderParametersBuilder Builder = FPrimitiveUniformShaderParametersBuilder{};
		BuildUniformShaderParameters(Builder);

		FPrimitiveUniformShaderParameters PrimitiveParams = Builder.Build();

		check(UniformBuffer);
		if (UniformBuffer.IsValid())
		{
			UniformBuffer.UpdateUniformBufferImmediate(RHICmdList, PrimitiveParams);
		}
	}
}

void FPrimitiveSceneProxy::BuildUniformShaderParameters(FPrimitiveUniformShaderParametersBuilder &Builder) const
{
	Builder = FPrimitiveUniformShaderParametersBuilder{}.Defaults();
	// 1. Set up parameters that depend on the proxy.
	BuildUniformShaderParametersInternal(Builder);
	// 2. Set up those that depend on renderer internal representations.
	Scene->BuildPrimitiveUniformShaderParameters_RenderThread(PrimitiveSceneInfo, Builder);
}

void FPrimitiveSceneProxy::BuildUniformShaderParametersInternal(FPrimitiveUniformShaderParametersBuilder& Builder) const
{
	FBoxSphereBounds PreSkinnedLocalBounds;
	GetPreSkinnedLocalBounds(PreSkinnedLocalBounds);

#if RHI_RAYTRACING
	const bool bHasRayTracingRepresentation = IsRayTracingEnabled()
		? HasRayTracingRepresentation() && IsVisibleInRayTracing()
		: HasDistanceFieldRepresentation();
#else
	const bool bHasRayTracingRepresentation = HasDistanceFieldRepresentation();
#endif

	// Update the uniform shader parameters.
	Builder.ActorWorldPosition(ActorPosition)
			.WorldBounds(Bounds)
			.LocalBounds(LocalBounds)
			.PreSkinnedLocalBounds(PreSkinnedLocalBounds)
			.ReceivesDecals(bReceivesDecals)
			.EvaluateWorldPositionOffset(EvaluateWorldPositionOffset() && AnyMaterialHasWorldPositionOffset())
			.MaxWorldPositionOffsetExtent(GetMaxWorldPositionOffsetExtent())
			.HasAlwaysEvaluateWPOMaterials(AnyMaterialAlwaysEvaluatesWorldPositionOffset())
			.MinMaxMaterialDisplacement(GetMinMaxMaterialDisplacement())
			.LightingChannelMask(GetLightingChannelMask())
			.LightmapUVIndex(GetLightMapCoordinateIndex())
			.CustomPrimitiveData(GetCustomPrimitiveData())
			.HasRayTracingRepresentation(bHasRayTracingRepresentation)
			.HasCapsuleRepresentation(HasDynamicIndirectShadowCasterRepresentation())
			.UseSingleSampleShadowFromStationaryLights(UseSingleSampleShadowFromStationaryLights())
			.CastContactShadow(CastsContactShadow())
			.CastHiddenShadow(CastsHiddenShadow())
			.CastShadow(CastsDynamicShadow())
			.Holdout(Holdout())
			.DisableMaterialInvalidations(ShadowCacheInvalidationBehavior == EShadowCacheInvalidationBehavior::Rigid || ShadowCacheInvalidationBehavior == EShadowCacheInvalidationBehavior::Static)
			.AllowInstanceCullingOcclusionQueries(AllowInstanceCullingOcclusionQueries())
			.VisibleInGame(IsDrawnInGame())
			.VisibleInEditor(IsDrawnInEditor())
			.VisibleInMainPass(ShouldRenderInMainPass())
			.VisibleInReflectionCaptures(IsVisibleInReflectionCaptures())
			.VisibleInRealTimeSkyCaptures(IsVisibleInRealTimeSkyCaptures())
			.VisibleInRayTracing(IsVisibleInRayTracing())
			.VisibleInLumenScene(IsVisibleInLumenScene() || IsLandscapeNaniteProxy())
			.VisibleInSceneCaptureOnly(IsVisibleInSceneCaptureOnly())
			.HiddenInSceneCapture(IsHiddenInSceneCapture())
			.ForceHidden(IsForceHidden())
			.PrimitiveComponentId(GetPrimitiveComponentId().PrimIDValue)
			.EditorColors(GetWireframeColor(), GetPrimitiveColor())
			.SplineMesh(IsSplineMesh())
			.SkinnedMesh(IsSkinnedMesh())
			.HasPixelAnimation(AnyMaterialHasPixelAnimation())
			.RayTracingFarField(IsRayTracingFarField())
			.RayTracingHasGroupId(GetRayTracingGroupId() != FPrimitiveSceneProxy::InvalidRayTracingGroupId)
			.MeshPaintTextureDescriptor(GetMeshPaintTextureDescriptor())
			.MaterialCacheDescriptor(GetMaterialCacheDescriptor())
			.IsFirstPerson(IsFirstPerson())
			.EnableObjectWorldPositionFix(!HasInstanceDataBuffers() && GEnableObjectWorldPositionFix)
			.AnimationMinScreenSize(GetAnimationMinScreenSize());

	if (IsNaniteMesh())
	{
		const Nanite::FSceneProxyBase* NaniteProxy = static_cast<const Nanite::FSceneProxyBase*>(this);

		Nanite::FResourcePrimitiveInfo NaniteResourcePrimitiveInfo = NaniteProxy->GetResourcePrimitiveInfo();
		uint32 NaniteFilterFlags = uint32(NaniteProxy->GetFilterFlags());
		uint32 NaniteRayTracingDataOffset = NaniteProxy->GetRayTracingDataOffset();
		bool bReverseCulling = NaniteProxy->IsCullingReversedByComponent(); // needed because Nanite doesn't use raster state
		float PixelProgrammableDistance = NaniteProxy->GetPixelProgrammableDistance();
		float MaterialDisplacementFadeOutSize = NaniteProxy->GetMaterialDisplacementFadeOutSize();
		
		Builder.NaniteResourceID(NaniteResourcePrimitiveInfo.ResourceID)
			.NaniteHierarchyOffset(NaniteResourcePrimitiveInfo.HierarchyOffset)
			.NaniteAssemblyInfo(NaniteResourcePrimitiveInfo.AssemblyTransformOffset, NaniteResourcePrimitiveInfo.AssemblyTransformCount)
			.NaniteFilterFlags(NaniteFilterFlags)
			.NaniteRayTracingDataOffset(NaniteRayTracingDataOffset)
			.ReverseCulling(bReverseCulling)
			.PixelProgrammableDistance(PixelProgrammableDistance)
			.MaterialDisplacementFadeOutSize(MaterialDisplacementFadeOutSize)
			.HasPerClusterDisplacementFallbackRaster(NaniteProxy->HasPerClusterDisplacementFallbackRaster());
	}


	FVector2f InstanceDrawDistanceMinMax;
	if (GetInstanceDrawDistanceMinMax(InstanceDrawDistanceMinMax))
	{
		Builder.InstanceDrawDistance(InstanceDrawDistanceMinMax);
	}

	// Note, always set the value to useful defaults (InstancingFadeOutParams.y == 0.0f means it works out to fully visible)
	FVector2f InstancingFadeOutParams = FVector2f::ZeroVector;
	GetInstancingFadeOutParams(InstancingFadeOutParams);
	Builder.InstancingFadeOutParams(InstancingFadeOutParams);

	float WPODisableDistance;
	if (GetInstanceWorldPositionOffsetDisableDistance(WPODisableDistance))
	{
		Builder.InstanceWorldPositionOffsetDisableDistance(WPODisableDistance);
	}

	if (HasInstanceDataBuffers())
	{
		const FInstanceSceneDataBuffers* InstanceSceneDataBuffers = GetInstanceSceneDataBuffers();
		if (GetInstanceDataHeader().NumInstances > 0)
		{
			// Getting the static mesh bounds from element 0 should always be valid, even if instance data is GPU-only.
			Builder.InstanceLocalBounds(InstanceSceneDataBuffers->GetInstanceLocalBounds(0));
		}
	}

	if (ShouldRenderCustomDepth())
	{
		Builder.CustomDepthStencil(GetCustomDepthStencilValue(), GetStencilWriteMask());
	}
}

void FPrimitiveSceneProxy::SetTransform(FRHICommandListBase& RHICmdList, const FMatrix& InLocalToWorld, const FBoxSphereBounds& InBounds, const FBoxSphereBounds& InLocalBounds, FVector InActorPosition)
{
	// Update the cached transforms.
	LocalToWorld = InLocalToWorld;
	bIsLocalToWorldDeterminantNegative = LocalToWorld.Determinant() < 0.0f;

	// Update the cached bounds. Pad them to account for max WPO and material displacement
	// TODO: DISP - Fix me
	const float PadAmount = GetAbsMaxDisplacement();
	Bounds = PadBounds(InBounds, PadAmount);
	LocalBounds = PadLocalBounds(InLocalBounds, LocalToWorld, PadAmount);
	ActorPosition = InActorPosition;
	
	// Update cached reflection capture.
	if (PrimitiveSceneInfo)
	{
		PrimitiveSceneInfo->bNeedsCachedReflectionCaptureUpdate = true;
		Scene->RequestUniformBufferUpdate(*PrimitiveSceneInfo);
	}

	// Notify the proxy's implementation of the change.
	OnTransformChanged(RHICmdList);
}

void FPrimitiveSceneProxy::UpdateInstances_RenderThread(FRHICommandListBase& RHICmdList, const FBoxSphereBounds& InBounds, const FBoxSphereBounds& InLocalBounds)
{
	TRACE_CPUPROFILER_EVENT_SCOPE_STR("FPrimitiveSceneProxy::UpdateInstances_RenderThread");
	// Update the cached bounds.
	Bounds = InBounds;
	LocalBounds = InLocalBounds;

	bAlwaysHasVelocity = CVarVelocityForceOutput.GetValueOnAnyThread() || GetInstanceDataHeader().Flags.bHasPerInstanceDynamicData || GetInstanceDataHeader().Flags.bHasPerInstanceSkinningData;
}

bool FPrimitiveSceneProxy::WouldSetTransformBeRedundant_AnyThread(const FMatrix& InLocalToWorld, const FBoxSphereBounds& InBounds, const FBoxSphereBounds& InLocalBounds, const FVector& InActorPosition) const
{
	// Order is based on cheapest tests first.
	// Actor position checks if the actor has moved in the world, and local bounds
	// then covers if the actor's size has changed. Other tests then follow suit.

	// Can be called by any thread, so be careful about modifying this.

	if (ActorPosition != InActorPosition)
	{
		return false;
	}

	// Account for padding that will be added to the bounds in SetTransform
	// TODO: DISP - Fix me
	const float PadAmount = GetAbsMaxDisplacement();

	if (LocalBounds != PadLocalBounds(InLocalBounds, InLocalToWorld, PadAmount))
	{
		return false;
	}

	if (Bounds != PadBounds(InBounds, PadAmount))
	{
		return false;
	}

	if (LocalToWorld != InLocalToWorld)
	{
		return false;
	}

	return true;
}

void FPrimitiveSceneProxy::ApplyWorldOffset(FRHICommandListBase& RHICmdList, FVector InOffset)
{
	FBoxSphereBounds NewBounds = FBoxSphereBounds(Bounds.Origin + InOffset, Bounds.BoxExtent, Bounds.SphereRadius);
	FBoxSphereBounds NewLocalBounds = LocalBounds;
	FVector NewActorPosition = ActorPosition + InOffset;
	FMatrix NewLocalToWorld = LocalToWorld.ConcatTranslation(InOffset);
	
	SetTransform(RHICmdList, NewLocalToWorld, NewBounds, NewLocalBounds, NewActorPosition);
}

void FPrimitiveSceneProxy::ApplyLateUpdateTransform(FRHICommandListBase& RHICmdList, const FMatrix& LateUpdateTransform)
{
	const FMatrix AdjustedLocalToWorld = LocalToWorld * LateUpdateTransform;
	SetTransform(RHICmdList, AdjustedLocalToWorld, Bounds, LocalBounds, ActorPosition);
}

bool FPrimitiveSceneProxy::UseSingleSampleShadowFromStationaryLights() const 
{ 
	return bSingleSampleShadowFromStationaryLights 
		|| CVarForceSingleSampleShadowingFromStationary.GetValueOnRenderThread() != 0
		|| LightmapType == ELightmapType::ForceVolumetric; 
}

bool FPrimitiveSceneProxy::IsInstanceDataGPUOnly() const
{
	return GetInstanceDataHeader().bInstanceDataIsGPUOnly;
}

bool FPrimitiveSceneProxy::UseEditorCompositing(const FSceneView* View) const
{
	return GIsEditor && bUseEditorCompositing && !View->bIsGameView;
}

#if ENABLE_DRAW_DEBUG
void FPrimitiveSceneProxy::SetDebugMassData(const TArray<FDebugMassData>& InDebugMassData)
{
	DebugMassData = InDebugMassData;
}
#endif

/**
 * Updates selection for the primitive proxy. This is called in the rendering thread by SetSelection_GameThread.
 * @param bInSelected - true if the parent actor is selected in the editor
 */
void FPrimitiveSceneProxy::SetSelection_RenderThread(const bool bInParentSelected, const bool bInIndividuallySelected)
{
	check(IsInParallelRenderingThread());

	const bool bWasSelected = IsSelected();
	bParentSelected = bInParentSelected;
	bIndividuallySelected = bInIndividuallySelected;
	const bool bIsSelected = IsSelected();

	if (bWantsEditorEffects == true)
	{
		GetScene().UpdatePrimitiveSelectedState_RenderThread(GetPrimitiveSceneInfo(), true);
		return;
	}

	// The renderer may have cached the selected state, let it know that this primitive is updated
	if ((bWasSelected && !bIsSelected) || 
		(bIsSelected && !bWasSelected))
	{
		GetScene().UpdatePrimitiveSelectedState_RenderThread(GetPrimitiveSceneInfo(), bIsSelected);
	}
}

/**
 * Updates selection for the primitive proxy. This simply sends a message to the rendering thread to call SetSelection_RenderThread.
 * This is called in the game thread as selection is toggled.
 * @param bInSelected - true if the parent actor is selected in the editor
 */
void FPrimitiveSceneProxy::SetSelection_GameThread(const bool bInParentSelected, const bool bInIndividuallySelected)
{
	check(IsInParallelGameThread() || IsInGameThread());

	// Enqueue a message to the rendering thread containing the interaction to add.
	FPrimitiveSceneProxy* PrimitiveSceneProxy = this;
	ENQUEUE_RENDER_COMMAND(SetNewSelection)(
		[PrimitiveSceneProxy, bInParentSelected, bInIndividuallySelected](FRHICommandListImmediate& RHICmdList)
		{
			PrimitiveSceneProxy->SetSelection_RenderThread(bInParentSelected, bInIndividuallySelected);
		});
}

void FPrimitiveSceneProxy::SetLevelInstanceEditingState_RenderThread(const bool bInEditingState)
{
	check(IsInRenderingThread());
	bLevelInstanceEditingState = bInEditingState;
}

void FPrimitiveSceneProxy::SetLevelInstanceEditingState_GameThread(const bool bInEditingState)
{
	check(IsInGameThread());

	// Enqueue a message to the rendering thread containing the interaction to add.
	FPrimitiveSceneProxy* PrimitiveSceneProxy = this;
	ENQUEUE_RENDER_COMMAND(SetNewLevelInstanceEditingState)(
		[PrimitiveSceneProxy, bInEditingState](FRHICommandListImmediate& RHICmdList)
		{
			PrimitiveSceneProxy->SetLevelInstanceEditingState_RenderThread(bInEditingState);
		});
}

/**
* Set the custom depth enabled flag
*
* @param the new value
*/
void FPrimitiveSceneProxy::SetCustomDepthEnabled_GameThread(const bool bInRenderCustomDepth)
{
	check(IsInGameThread());

	ENQUEUE_RENDER_COMMAND(FSetCustomDepthEnabled)(
		[this, bInRenderCustomDepth](FRHICommandList& RHICmdList)
		{
			this->SetCustomDepthEnabled_RenderThread(bInRenderCustomDepth);
		});
}

/**
* Set the custom depth enabled flag (RENDER THREAD)
*
* @param the new value
*/
void FPrimitiveSceneProxy::SetCustomDepthEnabled_RenderThread(const bool bInRenderCustomDepth)
{
	check(IsInRenderingThread());
	if (bRenderCustomDepth != bInRenderCustomDepth)
	{
		bRenderCustomDepth = bInRenderCustomDepth;

		if (PrimitiveSceneInfo)
		{
			Scene->RequestUniformBufferUpdate(*PrimitiveSceneInfo);

			if (IsNaniteMesh())
			{
				// We have to invalidate the primitive scene info's Nanite raster bins to refresh
				// whether or not they should render custom depth.
				Scene->RefreshNaniteRasterBins(*PrimitiveSceneInfo);
			}
		}
	}
}

/**
* Set the DrawInGame and the DrawInEditor flags (RENDER THREAD)
*
* @param the new value
*/
void FPrimitiveSceneProxy::SetIsDrawn_RenderThread(bool bIsDrawn)
{
	check(IsInRenderingThread());
	
	if ((DrawInGame != DrawInEditor) || (DrawInGame != bIsDrawn))
	{
		DrawInGame = bIsDrawn;
		DrawInEditor = bIsDrawn;

		if (PrimitiveSceneInfo)
		{
			Scene->RequestUniformBufferUpdate(*PrimitiveSceneInfo);
			Scene->RequestGPUSceneUpdate(*PrimitiveSceneInfo, EPrimitiveDirtyState::ChangedOther);	
		}
	}
}

/**
* Set the custom depth stencil value
*
* @param the new value
*/
void FPrimitiveSceneProxy::SetCustomDepthStencilValue_GameThread(const int32 InCustomDepthStencilValue)
{
	check(IsInGameThread());

	ENQUEUE_RENDER_COMMAND(FSetCustomDepthStencilValue)(
		[this, InCustomDepthStencilValue](FRHICommandList& RHICmdList)
	{
		this->SetCustomDepthStencilValue_RenderThread(InCustomDepthStencilValue);
	});
}

/**
* Set the custom depth stencil value (RENDER THREAD)
*
* @param the new value
*/
void FPrimitiveSceneProxy::SetCustomDepthStencilValue_RenderThread(const int32 InCustomDepthStencilValue)
{
	check(IsInRenderingThread());
	if (CustomDepthStencilValue != InCustomDepthStencilValue)
	{
		CustomDepthStencilValue = InCustomDepthStencilValue;

		if (PrimitiveSceneInfo)
		{
			Scene->RequestUniformBufferUpdate(*PrimitiveSceneInfo);
			Scene->RequestGPUSceneUpdate(*PrimitiveSceneInfo, EPrimitiveDirtyState::ChangedOther);	
		}
	}
}

void FPrimitiveSceneProxy::SetDistanceFieldSelfShadowBias_RenderThread(float NewBias)
{
	DistanceFieldSelfShadowBias = NewBias;
}

void FPrimitiveSceneProxy::SetDrawDistance_RenderThread(float InMinDrawDistance, float InMaxDrawDistance, float InVirtualTextureMaxDrawDistance)
{
	MinDrawDistance = InMinDrawDistance;
	MaxDrawDistance = InMaxDrawDistance > 0.0f ? InMaxDrawDistance : FLT_MAX;
	// Modify max draw distance for main pass if we are using virtual texturing
	if (WritesVirtualTexture() && InVirtualTextureMaxDrawDistance > 0.f)
	{
		MaxDrawDistance = FMath::Min(MaxDrawDistance, InVirtualTextureMaxDrawDistance);
	}
}

void FPrimitiveSceneProxy::GetPreSkinnedLocalBounds(FBoxSphereBounds& OutBounds) const
{
	// if we padded the local bounds for WPO, un-pad them for the "pre-skinned" bounds
	// (the idea being that WPO is a form of deformation, similar to skinning)
	// TODO: DISP - Fix me
	OutBounds = PadLocalBounds(LocalBounds, GetLocalToWorld(), -GetAbsMaxDisplacement());
}

/**
 * Updates hover state for the primitive proxy. This is called in the rendering thread by SetHovered_GameThread.
 * @param bInHovered - true if the parent actor is hovered
 */
void FPrimitiveSceneProxy::SetHovered_RenderThread(const bool bInHovered)
{
	check(IsInRenderingThread());
	bHovered = bInHovered;
}

/**
 * Updates hover state for the primitive proxy. This simply sends a message to the rendering thread to call SetHovered_RenderThread.
 * This is called in the game thread as hover state changes
 * @param bInHovered - true if the parent actor is hovered
 */
void FPrimitiveSceneProxy::SetHovered_GameThread(const bool bInHovered)
{
	check(IsInGameThread());

	// Enqueue a message to the rendering thread containing the interaction to add.
	FPrimitiveSceneProxy* PrimitiveSceneProxy = this;
	ENQUEUE_RENDER_COMMAND(SetNewHovered)(
		[PrimitiveSceneProxy, bInHovered](FRHICommandListImmediate& RHICmdList)
		{
			PrimitiveSceneProxy->SetHovered_RenderThread(bInHovered);
		});
}

void FPrimitiveSceneProxy::SetLightingChannels_GameThread(FLightingChannels LightingChannels)
{
	check(IsInGameThread());

	FPrimitiveSceneProxy* PrimitiveSceneProxy = this;
	const uint8 LocalLightingChannelMask = GetLightingChannelMaskForStruct(LightingChannels);
	ENQUEUE_RENDER_COMMAND(SetLightingChannelsCmd)(
		[PrimitiveSceneProxy, LocalLightingChannelMask, Scene = Scene, PrimitiveSceneInfo = PrimitiveSceneInfo](FRHICommandListImmediate& RHICmdList)
	{
		PrimitiveSceneProxy->LightingChannelMask = LocalLightingChannelMask;

		Scene->UpdateCachedRayTracingState(PrimitiveSceneProxy, EPrimitiveDirtyRayTracingState::Default | EPrimitiveDirtyRayTracingState::LightingChannelMask);

		if (PrimitiveSceneInfo)
		{
			Scene->RequestUniformBufferUpdate(*PrimitiveSceneInfo);
			Scene->RequestGPUSceneUpdate(*PrimitiveSceneInfo, EPrimitiveDirtyState::ChangedOther);
		}
	});
}

FRenderBounds FPrimitiveSceneProxy::PadInstanceLocalBounds(const FRenderBounds& InBounds)
{
	// TODO: DISP - Fix me
	return PadLocalRenderBounds(InBounds, GetLocalToWorld(), GetAbsMaxDisplacement());
}

#if ENABLE_DRAW_DEBUG
void FPrimitiveSceneProxy::FDebugMassData::DrawDebugMass(class FPrimitiveDrawInterface* PDI, const FTransform& ElemTM) const
{
	const FQuat MassOrientationToWorld = ElemTM.GetRotation() * LocalTensorOrientation;
	const FVector COMWorldPosition = ElemTM.TransformPosition(LocalCenterOfMass);

	const float Size = 15.f;
	const FVector XAxis = MassOrientationToWorld * FVector(1.f, 0.f, 0.f);
	const FVector YAxis = MassOrientationToWorld * FVector(0.f, 1.f, 0.f);
	const FVector ZAxis = MassOrientationToWorld * FVector(0.f, 0.f, 1.f);

	DrawCircle(PDI, COMWorldPosition, XAxis, YAxis, FColor(255, 255, 100), Size, 25, SDPG_World);
	DrawCircle(PDI, COMWorldPosition, ZAxis, YAxis, FColor(255, 255, 100), Size, 25, SDPG_World);

	const float InertiaSize = FMath::Max(MassSpaceInertiaTensor.Size(), UE_KINDA_SMALL_NUMBER);
	const float XSize = Size * MassSpaceInertiaTensor.X / InertiaSize;
	const float YSize = Size * MassSpaceInertiaTensor.Y / InertiaSize;
	const float ZSize = Size * MassSpaceInertiaTensor.Z / InertiaSize;

	const float Thickness = 2.f * FMath::Sqrt(3.f);	//We end up normalizing by inertia size. If the sides are all even we'll end up dividing by sqrt(3) since 1/sqrt(1+1+1)
	const float XThickness = Thickness * MassSpaceInertiaTensor.X / InertiaSize;
	const float YThickness = Thickness * MassSpaceInertiaTensor.Y / InertiaSize;
	const float ZThickness = Thickness * MassSpaceInertiaTensor.Z / InertiaSize;

	PDI->DrawLine(COMWorldPosition + XAxis * Size, COMWorldPosition - Size * XAxis, FColor(255, 0, 0), SDPG_World, XThickness);
	PDI->DrawLine(COMWorldPosition + YAxis * Size, COMWorldPosition - Size * YAxis, FColor(0, 255, 0), SDPG_World, YThickness);
	PDI->DrawLine(COMWorldPosition + ZAxis * Size, COMWorldPosition - Size * ZAxis, FColor(0, 0, 255), SDPG_World, ZThickness);
}
#endif

uint32 FPrimitiveSceneProxy::GetMaterialCacheDescriptor() const
{
	ensureMsgf(!bSupportsMaterialCache || MaterialCacheDescriptor.TagOffset != UINT16_MAX, TEXT("Primitive writes to material cache with an unallocated descriptor"));
	return MaterialCacheDescriptor.Opaque;
}

const FInstanceSceneDataBuffers *FPrimitiveSceneProxy::GetInstanceSceneDataBuffers(EInstanceBufferAccessFlags AccessFlags) const
{ 
	if (AccessFlags == EInstanceBufferAccessFlags::SynchronizeUpdateTask)
	{
		if (FInstanceDataUpdateTaskInfo *UpdateTaskInfo = GetInstanceDataUpdateTaskInfo())
		{
			UpdateTaskInfo->WaitForUpdateCompletion();
		}
	}
	return InstanceSceneDataBuffersInternal; 
}

FInstanceDataBufferHeader FPrimitiveSceneProxy::GetInstanceDataHeader() const
{
	if (FInstanceDataUpdateTaskInfo *UpdateTaskInfo = GetInstanceDataUpdateTaskInfo())
	{
		return UpdateTaskInfo->GetHeader();
	}

	if (InstanceSceneDataBuffersInternal)
	{
		return InstanceSceneDataBuffersInternal->GetHeader();
	}
	return FInstanceDataBufferHeader::SinglePrimitiveHeader;
}

void FPrimitiveSceneProxy::SetupInstanceSceneDataBuffers(const FInstanceSceneDataBuffers* InInstanceSceneDataBuffers)
{
	check(InstanceSceneDataBuffersInternal == nullptr);
	InstanceSceneDataBuffersInternal = InInstanceSceneDataBuffers;
}

bool FPrimitiveSceneProxy::DrawInRuntimeVirtualTextureOnly(bool bEditor) const
{
	if (bVirtualTextureMainPassDrawAlways)
	{
		return false;
	}
	else if (bVirtualTextureMainPassDrawNever)
	{
		return true;
	}
	// Conditional path tests against the collected flags from all runtime virtual textures in the scene.
 	uint8 bHideMaskEditor, bHideMaskGame;
 	Scene->GetRuntimeVirtualTextureHidePrimitiveMask(bHideMaskEditor, bHideMaskGame);
 	const uint8 bHideMask = bEditor ? bHideMaskEditor : bHideMaskGame;
 	return bHideMask != 0;
}

void FPrimitiveSceneProxy::EnableGPUSceneSupportFlags()
{
	const EShaderPlatform ShaderPlatform = GetScene().GetShaderPlatform();
	const bool bUseGPUScene = UseGPUScene(ShaderPlatform);
	const bool bMobilePath = IsMobilePlatform(ShaderPlatform);

	// Skip primitive uniform buffer if we will be using local vertex factory which gets it's data from GPUScene.
	bVFRequiresPrimitiveUniformBuffer = !bUseGPUScene;
	// For mobile we always assume that proxy does not support GPUScene, as it depends on vertex factory setup which happens later
	bSupportsGPUScene = bMobilePath ? false : bUseGPUScene;
}

/**
 * Updates the hidden editor view visibility map on the game thread which just enqueues a command on the render thread
 */
void FPrimitiveSceneProxy::SetHiddenEdViews_GameThread( uint64 InHiddenEditorViews )
{
	check(IsInGameThread());

	FPrimitiveSceneProxy* PrimitiveSceneProxy = this;
	ENQUEUE_RENDER_COMMAND(SetEditorVisibility)(
		[PrimitiveSceneProxy, InHiddenEditorViews](FRHICommandListImmediate& RHICmdList)
		{
			PrimitiveSceneProxy->SetHiddenEdViews_RenderThread(InHiddenEditorViews);
		});
}

/**
 * Updates the hidden editor view visibility map on the render thread 
 */
void FPrimitiveSceneProxy::SetHiddenEdViews_RenderThread( uint64 InHiddenEditorViews )
{
#if WITH_EDITOR
	check(IsInRenderingThread());
	HiddenEditorViews = InHiddenEditorViews;
#endif
}

#if WITH_EDITOR
void FPrimitiveSceneProxy::SetIsBeingMovedByEditor_GameThread(bool bIsBeingMoved)
{
	check(IsInGameThread());

	FPrimitiveSceneProxy* PrimitiveSceneProxy = this;
	ENQUEUE_RENDER_COMMAND(SetIsBeingMovedByEditor)(
		[PrimitiveSceneProxy, bIsBeingMoved](FRHICommandListImmediate& RHICmdList)
		{
			PrimitiveSceneProxy->bIsBeingMovedByEditor = bIsBeingMoved;
			PrimitiveSceneProxy->GetScene().UpdatePrimitiveVelocityState_RenderThread(PrimitiveSceneProxy->GetPrimitiveSceneInfo(), bIsBeingMoved);
		});
}

void FPrimitiveSceneProxy::SetSelectionOverride_GameThread(bool bForceSelection)
{
	check(IsInGameThread());

	ENQUEUE_RENDER_COMMAND(SetSelectionOutlineColorIndex)(
		[this, bForceSelection](FRHICommandListImmediate&)
		{
			bWantsEditorEffects = bForceSelection;
			SetSelection_RenderThread(bParentSelected, bIndividuallySelected);
		});
}

void FPrimitiveSceneProxy::SetSelectionOutlineColorIndex_GameThread(uint8 ColorIndex)
{
	check(IsInGameThread());
	constexpr int IndexBits = 3;
	constexpr int MaxIndex = (1 << IndexBits) - 1;
	check(ColorIndex <= MaxIndex);

	ENQUEUE_RENDER_COMMAND(SetSelectionOutlineColorIndex)(
		[this, ColorIndex](FRHICommandListImmediate&)
		{
			SelectionOutlineColorIndex = ColorIndex;
		});
}

void FPrimitiveSceneProxy::SetOverlayColor_GameThread(FColor InOverlayColor)
{
	check(IsInGameThread());

	ENQUEUE_RENDER_COMMAND(SetSelectionOutlineColorIndex)(
		[this, InOverlayColor](FRHICommandListImmediate&)
		{
			OverlayColor = InOverlayColor;
		});
}
#endif

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
void FPrimitiveSceneProxy::SetPrimitiveColor_GameThread(const FLinearColor& InPrimitiveColor)
{
	check(IsInGameThread());

	ENQUEUE_RENDER_COMMAND(SetSelectionOutlineColorIndex)(
		[this, InPrimitiveColor](FRHICommandListImmediate&)
		{
			if (PrimitiveColor != InPrimitiveColor)
			{
				PrimitiveColor = InPrimitiveColor;

				if (PrimitiveSceneInfo)
				{
					Scene->RequestUniformBufferUpdate(*PrimitiveSceneInfo);
					Scene->RequestGPUSceneUpdate(*PrimitiveSceneInfo, EPrimitiveDirtyState::ChangedOther);
				}
			}
		});
}
#endif

void FPrimitiveSceneProxy::ResetSceneVelocity_GameThread()
{
	check(IsInGameThread());

	FPrimitiveSceneProxy* PrimitiveSceneProxy = this;
	ENQUEUE_RENDER_COMMAND(ResetSceneVelocity)(
		[PrimitiveSceneProxy](FRHICommandListImmediate& RHICmdList)
		{
			PrimitiveSceneProxy->GetScene().UpdatePrimitiveVelocityState_RenderThread(PrimitiveSceneProxy->GetPrimitiveSceneInfo(), false);
		});
}

void FPrimitiveSceneProxy::SetEvaluateWorldPositionOffset_GameThread(bool bEvaluate)
{
	check(IsInGameThread());

	FPrimitiveSceneProxy* PrimitiveSceneProxy = this;
	ENQUEUE_RENDER_COMMAND(SetEvaluateWorldPositionOffset)
		([PrimitiveSceneProxy, bEvaluate, Scene = Scene, PrimitiveSceneInfo = PrimitiveSceneInfo](FRHICommandList& RHICmdList)
	{
		if (PrimitiveSceneProxy->bEvaluateWorldPositionOffset != bEvaluate)
		{
			PrimitiveSceneProxy->bEvaluateWorldPositionOffset = bEvaluate;

			if (PrimitiveSceneInfo)
			{
				Scene->RequestUniformBufferUpdate(*PrimitiveSceneInfo);
				Scene->RequestGPUSceneUpdate(*PrimitiveSceneInfo, EPrimitiveDirtyState::ChangedOther);
			}

			PrimitiveSceneProxy->OnEvaluateWorldPositionOffsetChanged_RenderThread();
		}
	});
}

ESceneDepthPriorityGroup FPrimitiveSceneProxy::GetDepthPriorityGroup(const FSceneView* View) const
{
	return (bUseViewOwnerDepthPriorityGroup && IsOwnedBy(View->ViewActor)) ?
		(ESceneDepthPriorityGroup)ViewOwnerDepthPriorityGroup :
		(ESceneDepthPriorityGroup)StaticDepthPriorityGroup;
}

void FPrimitiveSceneProxy::SetCollisionEnabled_GameThread(const bool bNewEnabled)
{
	check(IsInGameThread());

	// Enqueue a message to the rendering thread to change draw state
	FPrimitiveSceneProxy* PrimSceneProxy = this;
	ENQUEUE_RENDER_COMMAND(SetCollisionEnabled)(
		[PrimSceneProxy, bNewEnabled](FRHICommandListImmediate& RHICmdList)
		{
			PrimSceneProxy->SetCollisionEnabled_RenderThread(bNewEnabled);
		});
}

void FPrimitiveSceneProxy::SetCollisionEnabled_RenderThread(const bool bNewEnabled)
{
	check(IsInRenderingThread());
	bCollisionEnabled = bNewEnabled;
}

/** @return True if the primitive is visible in the given View. */
bool FPrimitiveSceneProxy::IsShown(const FSceneView* View) const
{
	// If primitive is forcibly hidden
	if (IsForceHidden())
	{
		return false;
	}

#if WITH_EDITOR
	// Don't draw editor specific actors during game mode
	if (View->Family->EngineShowFlags.Game)
	{
		if (DrawInAnyEditMode)
		{
			return false;
		}
	}

	if (bIsFoliage && !View->Family->EngineShowFlags.InstancedFoliage)
	{
		return false;
	}

	// After checking for VR/Desktop Edit mode specific actors, check for Editor vs. Game
	if(View->Family->EngineShowFlags.Editor)
	{
		if(!DrawInEditor)
		{
			return false;
		}

		// if all of it's groups are hidden in this view, don't draw
		if ((HiddenEditorViews & View->EditorViewBitflag) != 0)
		{
			return false;
		}

		// If we are in a collision view, hide anything which doesn't have collision enabled
		const bool bCollisionView = (View->Family->EngineShowFlags.CollisionVisibility || View->Family->EngineShowFlags.CollisionPawn);
		if (bCollisionView && !IsCollisionEnabled())
		{
			return false;
		}

		if (DrawInRuntimeVirtualTextureOnly(true) && !View->bIsVirtualTexture && !View->Family->EngineShowFlags.VirtualTexturePrimitives && !IsSelected())
		{
			return false;
		}
	}
	else
#endif
	{

		if(!DrawInGame
#if WITH_EDITOR
			|| (!View->bIsGameView && View->Family->EngineShowFlags.Game && !DrawInEditor)	// ..."G" mode in editor viewport. covers the case when the primitive must be rendered for the voxelization pass, but the user has chosen to hide the primitive from view.
#endif
			)
		{
			return false;
		}

		if (DrawInRuntimeVirtualTextureOnly(false) && !View->bIsVirtualTexture)
		{
			return false;
		}

		const bool bOwnersContain = View->ViewActor.IsPartOf(Owners);
		if (bOnlyOwnerSee && !bOwnersContain)
		{
			return false;
		}

		if (bOwnerNoSee && bOwnersContain)
		{
			return false;
		}
	}

	if (View->bIsSceneCapture)
	{
		if (IsHiddenInSceneCapture())
			return false;
	}
	else
	{
		if (IsVisibleInSceneCaptureOnly())
			return false;
	}

	return true;
}

/** @return True if the primitive is casting a shadow. */
bool FPrimitiveSceneProxy::IsShadowCast(const FSceneView* View) const
{
	check(PrimitiveSceneInfo);

	if (!CastsStaticShadow() && !CastsDynamicShadow())
	{
		return false;
	}

	if (!CastsHiddenShadow())
	{
		// Primitives that are hidden in the game don't cast a shadow.
		if (!DrawInGame)
		{
			return false;
		}

		if (!View->HiddenPrimitives.IsEmpty() && View->HiddenPrimitives.Contains(PrimitiveComponentId))
		{
			return false;
		}

		if (View->ShowOnlyPrimitives.IsSet() && !View->ShowOnlyPrimitives->Contains(PrimitiveComponentId))
		{
			return false;
		}

#if WITH_EDITOR
		// For editor views, we use a show flag to determine whether shadows from editor-hidden actors are desired.
		if( View->Family->EngineShowFlags.Editor )
		{
			if(!DrawInEditor)
			{
				return false;
			}
		
			// if all of it's groups are hidden in this view, don't draw
			if ((HiddenEditorViews & View->EditorViewBitflag) != 0)
			{
				return false;
			}
		}
#endif	//#if WITH_EDITOR

		// If primitive is forcibly hidden
		if (IsForceHidden())
		{
			return false;
		}

		if (DrawInRuntimeVirtualTextureOnly(View->Family->EngineShowFlags.Editor) && !View->bIsVirtualTexture)
		{
			return false;
		}

		// In the OwnerSee cases, we still want to respect hidden shadows...
		// This assumes that bCastHiddenShadow trumps the owner see flags.
		const bool bOwnersContain = View->ViewActor.IsPartOf(Owners);
		if (bOnlyOwnerSee && !bOwnersContain)
		{
			return false;
		}

		if (bOwnerNoSee && bOwnersContain)
		{
			return false;
		}

		if (View->bIsSceneCapture && IsHiddenInSceneCapture())
		{
			return false;
		}
	}

	if (!View->bIsSceneCapture && IsVisibleInSceneCaptureOnly())
	{
		return false;
	}

	return true;
}

void FPrimitiveSceneProxy::UpdateVisibleInLumenScene()
{
	bool bLumenUsesHardwareRayTracing = false;
#if RHI_RAYTRACING
	static const auto LumenUseHardwareRayTracingCVar = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.Lumen.HardwareRayTracing"));
	static const auto LumenUseFarFieldCVar = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.LumenScene.FarField"));
	static const auto LumenUseFarFieldOcclusionOnlyCVar = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.LumenScene.FarField.OcclusionOnly"));
	bLumenUsesHardwareRayTracing = IsRayTracingEnabled()
		&& (GRHISupportsRayTracingShaders || GRHISupportsInlineRayTracing)
		&& LumenUseHardwareRayTracingCVar->GetValueOnAnyThread() != 0;
#endif

	bool bAffectsLumen = AffectsDynamicIndirectLighting();
	bool bCanBeTraced = false;
	if (bLumenUsesHardwareRayTracing)
	{
#if RHI_RAYTRACING
		if (IsRayTracingAllowed() && HasRayTracingRepresentation())
		{
			if ((IsVisibleInRayTracing() && (IsDrawnInGame() || AffectsIndirectLightingWhileHidden())) 
				|| (IsRayTracingFarField() && LumenUseFarFieldCVar->GetValueOnAnyThread() != 0))
			{
				bCanBeTraced = true;
			}

			if (IsRayTracingFarField() && LumenUseFarFieldOcclusionOnlyCVar->GetValueOnAnyThread() != 0)
			{
				bAffectsLumen = false;
			}
		}
#endif
	}
	else
	{
		if (DoesProjectSupportDistanceFields() && AffectsDistanceFieldLighting() && (SupportsDistanceFieldRepresentation() || SupportsHeightfieldRepresentation()))
		{
			if (IsDrawnInGame() || AffectsIndirectLightingWhileHidden())
			{
				bCanBeTraced = true;
			}
		}
	}

	bVisibleInLumenScene = bAffectsLumen && bCanBeTraced;
}

void FPrimitiveSceneProxy::RenderBounds(
	FPrimitiveDrawInterface* PDI, 
	const FEngineShowFlags& EngineShowFlags, 
	const FBoxSphereBounds& InBounds, 
	bool bRenderInEditor) const
{
	if (EngineShowFlags.Bounds && (EngineShowFlags.Game || bRenderInEditor))
	{
		// Draw the static mesh's bounding box and sphere.
		const ESceneDepthPriorityGroup DrawBoundsDPG = SDPG_World;
		DrawWireBox(PDI,InBounds.GetBox(), FColor(72,72,255), DrawBoundsDPG);
		DrawCircle(PDI, InBounds.Origin, FVector(1, 0, 0), FVector(0, 1, 0), FColor::Yellow, InBounds.SphereRadius, 32, DrawBoundsDPG);
		DrawCircle(PDI, InBounds.Origin, FVector(1, 0, 0), FVector(0, 0, 1), FColor::Yellow, InBounds.SphereRadius, 32, DrawBoundsDPG);
		DrawCircle(PDI, InBounds.Origin, FVector(0, 1, 0), FVector(0, 0, 1), FColor::Yellow, InBounds.SphereRadius, 32, DrawBoundsDPG);
		// render the Local bounds in a red color
		DrawWireBox(PDI, GetLocalToWorld(), GetLocalBounds().GetBox(), FColor(255, 72, 72), DrawBoundsDPG);
	}
}

bool FPrimitiveSceneProxy::VerifyUsedMaterial(const FMaterialRenderProxy* MaterialRenderProxy) const
{
	// Only verify GetUsedMaterials if uncooked and we can compile shaders, because FShaderCompilingManager::PropagateMaterialChangesToPrimitives is what needs GetUsedMaterials to be accurate
#if WITH_EDITOR
	if (bVerifyUsedMaterials)
	{
		const UMaterialInterface* MaterialInterface = MaterialRenderProxy->GetMaterialInterface();

		if (MaterialInterface 
			&& !UsedMaterialsForVerification.Contains(MaterialInterface)
			&& MaterialInterface != UMaterial::GetDefaultMaterial(MD_Surface)
			&& MaterialInterface != GEngine->NaniteHiddenSectionMaterial)
		{
			// Shader compiling uses GetUsedMaterials to detect which components need their scene proxy recreated, so we can only render with materials present in that list
			ensureMsgf(false, TEXT("PrimitiveComponent tried to render with Material %s, which was not present in the component's GetUsedMaterials results\n    Owner: %s, Resource: %s"), *MaterialInterface->GetName(), *GetOwnerName().ToString(), *GetResourceName().ToString());
			return false;
		}
	}
#endif
	return true;
}

void FPrimitiveSceneProxy::DrawArc(FPrimitiveDrawInterface* PDI, const FVector& Start, const FVector& End, const float Height, const uint32 Segments, const FLinearColor& Color, uint8 DepthPriorityGroup, const float Thickness, const bool bScreenSpace)
{
	if (Segments == 0)
	{
		return;
	}

	const float ARC_PTS_SCALE = 1.0f / (float)Segments;
	const float DepthBias = 0.0f;

	const float X0 = Start.X;
	const float Y0 = Start.Y;
	const float Z0 = Start.Z;
	const float Dx = End.X - X0;
	const float Dy = End.Y - Y0;
	const float Dz = End.Z - Z0;
	const float Length = FMath::Sqrt(Dx*Dx + Dy*Dy + Dz*Dz);
	float Px = X0, Py = Y0, Pz = Z0;
	for (uint32 i = 1; i <= Segments; ++i)
	{
		const float U = i * ARC_PTS_SCALE;
		const float X = X0 + Dx * U;
		const float Y = Y0 + Dy * U;
		const float Z = Z0 + Dz * U + (Length*Height) * (1-(U*2-1)*(U*2-1));

		PDI->DrawLine( FVector(Px, Py, Pz), FVector(X, Y, Z), Color, SDPG_World, Thickness, DepthBias, bScreenSpace);

		Px = X; Py = Y; Pz = Z;
	}
}

void FPrimitiveSceneProxy::DrawArrowHead(FPrimitiveDrawInterface* PDI, const FVector& Tip, const FVector& Origin, const float Size, const FLinearColor& Color, uint8 DepthPriorityGroup, const float Thickness, const bool bScreenSpace)
{
	const FVector Forward = (Origin - Tip).GetUnsafeNormal();
	const FVector Right = FVector::CrossProduct(Forward, FVector::UpVector);
	const float HalfWidth = Size / 3.0f;
	const float DepthBias = 0.0f;
	PDI->DrawLine(Tip, Tip + Forward * Size + Right * HalfWidth, Color, DepthPriorityGroup, Thickness, DepthBias, bScreenSpace);
	PDI->DrawLine(Tip, Tip + Forward * Size - Right * HalfWidth, Color, DepthPriorityGroup, Thickness, DepthBias, bScreenSpace);
}


#if WITH_EDITORONLY_DATA
bool FPrimitiveSceneProxy::GetPrimitiveDistance(int32 LODIndex, int32 SectionIndex, const FVector& ViewOrigin, float& PrimitiveDistance) const
{
	const bool bUseNewMetrics = CVarStreamingUseNewMetrics.GetValueOnRenderThread() != 0;

	const FBoxSphereBounds& PrimBounds = GetBounds();

	FVector ViewToObject = PrimBounds.Origin - ViewOrigin;

	float DistSqMinusRadiusSq = 0;
	if (bUseNewMetrics)
	{
		ViewToObject = ViewToObject.GetAbs();
		FVector BoxViewToObject = ViewToObject.ComponentMin(PrimBounds.BoxExtent);
		DistSqMinusRadiusSq = FVector::DistSquared(BoxViewToObject, ViewToObject);
	}
	else
	{
		float Distance = ViewToObject.Size();
		DistSqMinusRadiusSq = FMath::Square(Distance) - FMath::Square(PrimBounds.SphereRadius);
	}

	PrimitiveDistance = FMath::Sqrt(FMath::Max<float>(1.f, DistSqMinusRadiusSq));
	return true;
}

bool FPrimitiveSceneProxy::GetMeshUVDensities(int32 LODIndex, int32 SectionIndex, FVector4& WorldUVDensities) const 
{ 
	return false; 
}

bool FPrimitiveSceneProxy::GetMaterialTextureScales(int32 LODIndex, int32 SectionIndex, const FMaterialRenderProxy* MaterialRenderProxy, FVector4f* OneOverScales, FIntVector4* UVChannelIndices) const 
{ 
	return false; 
}

#endif // WITH_EDITORONLY_DATA

#if UE_WITH_PSO_PRECACHING
void FPrimitiveSceneProxy::BoostPrecachedPSORequestsOnDraw()
{
	if (!PSOPrecacheRequestsToBoostOnDraw.IsEmpty())
	{
		TArray<FMaterialPSOPrecacheRequestID> LocalPSOPrecacheRequestsToBoostOnDraw = MoveTemp(PSOPrecacheRequestsToBoostOnDraw);
		BoostPSOPriorityOnDraw(LocalPSOPrecacheRequestsToBoostOnDraw);
	}
}
#endif // UE_WITH_PSO_PRECACHING

#if RHI_RAYTRACING
ERayTracingPrimitiveFlags FPrimitiveSceneProxy::GetRayTracingPrimitiveFlags()
{
	if (!IsRayTracingRelevant())
	{
		// The entire proxy type will be skipped. Make sure IsRayTracingRelevant() only depends on proxy type (vtable)
		return ERayTracingPrimitiveFlags::UnsupportedProxyType;
	}

	if (!HasRayTracingRepresentation())
	{
		return ERayTracingPrimitiveFlags::Exclude;
	}

	// TODO: Ideally all these other conditions should be folded into HasRayTracingRepresentation() 
	// to avoid primitives being unecessarily registered with FRayTracingGeometryManager / Nanite::FRayTracingManager
	// however would need to make sure they would get registered/unregistered as the conditions change

	if (!(IsVisibleInRayTracing() && ShouldRenderInMainPass() && (IsDrawnInGame() || AffectsIndirectLightingWhileHidden() || CastsHiddenShadow())) && !IsRayTracingFarField())
	{
		return ERayTracingPrimitiveFlags::Exclude;
	}

	static const auto RayTracingStaticMeshesCVar = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.RayTracing.Geometry.StaticMeshes"));

	if ((bIsStaticMesh || bIsNaniteMesh) && RayTracingStaticMeshesCVar && RayTracingStaticMeshesCVar->GetValueOnRenderThread() <= 0)
	{
		return ERayTracingPrimitiveFlags::Exclude;
	}

	static const auto RayTracingSkeletalMeshesCVar = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.RayTracing.Geometry.SkeletalMeshes"));

	if (bSkinnedMesh && RayTracingSkeletalMeshesCVar && RayTracingSkeletalMeshesCVar->GetValueOnRenderThread() <= 0)
	{
		return ERayTracingPrimitiveFlags::Exclude;
	}

	static const auto RayTracingHISMCVar = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.RayTracing.Geometry.HierarchicalInstancedStaticMesh"));

	if (bIsHierarchicalInstancedStaticMesh && RayTracingHISMCVar && RayTracingHISMCVar->GetValueOnRenderThread() <= 0)
	{
		return ERayTracingPrimitiveFlags::Exclude;
	}

	static const auto RayTracingLandscapeGrassCVar = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.RayTracing.Geometry.LandscapeGrass"));

	if (bIsLandscapeGrass && RayTracingLandscapeGrassCVar && RayTracingLandscapeGrassCVar->GetValueOnRenderThread() <= 0)
	{
		return ERayTracingPrimitiveFlags::Exclude;
	}

	if (IsFirstPerson())
	{
		// First person primitives are currently not supported in raytracing as this kind of geometry only makes sense from the camera's point of view.
		return ERayTracingPrimitiveFlags::Exclude;
	}

	// Visible in ray tracing. 
	ERayTracingPrimitiveFlags ResultFlags = ERayTracingPrimitiveFlags::None;

	if (IsRayTracingStaticRelevant())
	{
		ResultFlags |= ERayTracingPrimitiveFlags::ComputeLOD;
	}
	else
	{
		// Fully dynamic (no caching)
		ResultFlags |= ERayTracingPrimitiveFlags::Dynamic;
	}

	if (IsRayTracingFarField())
	{
		ResultFlags |= ERayTracingPrimitiveFlags::FarField;
	}

	return ResultFlags;
}

void FPrimitiveSceneProxy::GetStaticRayTracingInstances(FStaticRayTracingInstances& OutRayTracingInstances)
{
#if DO_CHECK
	ERayTracingPrimitiveFlags Flags = GetRayTracingPrimitiveFlags();

	if (EnumHasAnyFlags(Flags, ERayTracingPrimitiveFlags::CacheInstances))
	{
		checkf(false, TEXT("Proxy classes returning ERayTracingPrimitiveFlags::CacheInstances must override GetStaticRayTracingInstances(...)"));
	}
#endif
}

ERayTracingPrimitiveFlags FPrimitiveSceneProxy::GetCachedRayTracingInstance(FRayTracingInstance& OutRayTracingInstance)
{
	ERayTracingPrimitiveFlags Flags = GetRayTracingPrimitiveFlags();
	FStaticRayTracingInstances StaticRayTracingInstances;
	GetStaticRayTracingInstances(StaticRayTracingInstances);
	if (StaticRayTracingInstances.LODs.Num() > 0 && StaticRayTracingInstances.LODs[0].Instances.Num() > 0)
	{
		OutRayTracingInstance = StaticRayTracingInstances.LODs[0].Instances[0];
	}
	return Flags;
}

ERayTracingProxyType FPrimitiveSceneProxy::GetRayTracingProxyType() const
{
	ERayTracingProxyType ProxyType = ERayTracingProxyType::None;
	if (bIsStaticMesh || bIsNaniteMesh)
	{
		EnumAddFlags(ProxyType, ERayTracingProxyType::StaticMesh);
	}
	if (bIsHierarchicalInstancedStaticMesh)
	{
		EnumAddFlags(ProxyType, ERayTracingProxyType::HierarchicalInstancedStaticMesh);
	}
	if (bIsLandscapeGrass)
	{
		EnumAddFlags(ProxyType, ERayTracingProxyType::LandscapeGrass);
	}
	if (bSkinnedMesh)
	{
		EnumAddFlags(ProxyType, ERayTracingProxyType::SkeletalMesh);
	}
	if (bInstancedSkinnedMesh)
	{
		EnumAddFlags(ProxyType, ERayTracingProxyType::InstanceSkeletalMesh);
	}
	return ProxyType;
}
#endif // RHI_RAYTRACING
