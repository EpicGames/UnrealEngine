// Copyright Epic Games, Inc. All Rights Reserved.

#include "PrimitiveSceneProxyDesc.h"
#include "Components/InstancedStaticMeshComponent.h"
#include "GameFramework/PrimitiveComponentUtilities.h"
#include "SceneInterface.h"

#if WITH_EDITOR
#include "FoliageHelper.h"
#endif

FPrimitiveSceneProxyDesc::FPrimitiveSceneProxyDesc()
{
	CastShadow = false;
	bReceivesDecals = true;
	bOnlyOwnerSee = false;
	bOwnerNoSee = false;
	bLevelInstanceEditingState = false;
	bUseViewOwnerDepthPriorityGroup = false;
	bVisibleInReflectionCaptures = true;
	bVisibleInRealTimeSkyCaptures = true;
	bVisibleInRayTracing = true;
	bVisibleInReflections = true;
	bRenderInDepthPass = true;
	bRenderInMainPass = true;
	bTreatAsBackgroundForOcclusion = false;
	bCastDynamicShadow = true;
	bCastStaticShadow = true;
	bEmissiveLightSource = false;
	bAffectDynamicIndirectLighting = true;
	bAffectIndirectLightingWhileHidden = false;
	bAffectDistanceFieldLighting = true;
	bCastVolumetricTranslucentShadow = false;
	bCastContactShadow = true;
	bCastHiddenShadow = false;
	bCastShadowAsTwoSided = false;
	bSelfShadowOnly = false;
	bCastInsetShadow = false;
	bCastCinematicShadow = false;
	bCastFarShadow = false;
	bLightAttachmentsAsGroup = false;
	bSingleSampleShadowFromStationaryLights = false;
	bUseAsOccluder = false;
	bSelectable = true;
	bHasPerInstanceHitProxies = false;
	bUseEditorCompositing = false;
	bIsBeingMovedByEditor = false;
	bReceiveMobileCSMShadows = true;
	bRenderCustomDepth = false;
	bVisibleInSceneCaptureOnly = false;
	bHiddenInSceneCapture = false;
	bForceMipStreaming = false;
	bRayTracingFarField = false;
	bIsVisible = true;
	bIsVisibleEditor = true;
	bSelected = false;
	bIndividuallySelected = false;
	bCollisionEnabled = false;
	bIsHidden = false;
	bIsHiddenEd = false;
	bSupportsWorldPositionOffsetVelocity = true;
	bIsOwnerEditorOnly = false;
	bIsInstancedStaticMesh = false;
	bHoldout = false;
	bIsFirstPerson = false;
	bIsFirstPersonWorldSpaceRepresentation = false;
	bLumenHeightfield = false;

	bHasStaticLighting = false;
	bHasValidSettingsForStaticLighting = false;
	bIsPrecomputedLightingValid = false;
	bShadowIndirectOnly = false;
	bUsePSOPrecacheFallbackMaterial = false;
	bShouldRenderSelected = false;
	bWantsEditorEffects = false;

#if WITH_EDITOR
	bIsOwnedByFoliage = false;
#endif
}

FPrimitiveSceneProxyDesc::~FPrimitiveSceneProxyDesc() = default;

void FPrimitiveSceneProxyDesc::InitializeFromPrimitiveComponent(const UPrimitiveComponent* InComponent)
{
	CastShadow = InComponent->CastShadow;
	bReceivesDecals = InComponent->bReceivesDecals;
	bOnlyOwnerSee = InComponent->bOnlyOwnerSee;
	bOwnerNoSee = InComponent->bOwnerNoSee;
	bLevelInstanceEditingState = InComponent->GetLevelInstanceEditingState();
	bUseViewOwnerDepthPriorityGroup = InComponent->bUseViewOwnerDepthPriorityGroup;
	bVisibleInReflectionCaptures = InComponent->bVisibleInReflectionCaptures;
	bVisibleInRealTimeSkyCaptures = InComponent->bVisibleInRealTimeSkyCaptures;
	bVisibleInRayTracing = InComponent->bVisibleInRayTracing;
	bVisibleInReflections = InComponent->bVisibleInReflections;
	bRenderInDepthPass = InComponent->bRenderInDepthPass;
	bRenderInMainPass = InComponent->bRenderInMainPass;
	bTreatAsBackgroundForOcclusion = InComponent->bTreatAsBackgroundForOcclusion;
	bCastDynamicShadow = InComponent->bCastDynamicShadow;
	bCastStaticShadow = InComponent->bCastStaticShadow;
	bEmissiveLightSource = InComponent->bEmissiveLightSource;
	bAffectDynamicIndirectLighting = InComponent->bAffectDynamicIndirectLighting;
	bAffectIndirectLightingWhileHidden = InComponent->bAffectIndirectLightingWhileHidden;
	bAffectDistanceFieldLighting = InComponent->bAffectDistanceFieldLighting;
	bCastVolumetricTranslucentShadow = InComponent->bCastVolumetricTranslucentShadow;
	bCastContactShadow = InComponent->bCastContactShadow;
	bCastHiddenShadow = InComponent->bCastHiddenShadow;
	bCastShadowAsTwoSided = InComponent->bCastShadowAsTwoSided;
	bSelfShadowOnly = InComponent->bSelfShadowOnly;
	bCastInsetShadow = InComponent->bCastInsetShadow;
	bCastCinematicShadow = InComponent->bCastCinematicShadow;
	bCastFarShadow = InComponent->bCastFarShadow;
	bLightAttachmentsAsGroup = InComponent->bLightAttachmentsAsGroup;
	bSingleSampleShadowFromStationaryLights = InComponent->bSingleSampleShadowFromStationaryLights;
	bUseAsOccluder = InComponent->bUseAsOccluder;
	bSelectable = InComponent->bSelectable;
	bHasPerInstanceHitProxies = InComponent->bHasPerInstanceHitProxies;
	bUseEditorCompositing = InComponent->bUseEditorCompositing;
	bIsBeingMovedByEditor = InComponent->bIsBeingMovedByEditor;
	bReceiveMobileCSMShadows = InComponent->bReceiveMobileCSMShadows;
	bRenderCustomDepth = InComponent->bRenderCustomDepth;
	bVisibleInSceneCaptureOnly = InComponent->bVisibleInSceneCaptureOnly;
	bHiddenInSceneCapture = InComponent->bHiddenInSceneCapture;
	bForceMipStreaming = InComponent->bForceMipStreaming;
	bRayTracingFarField = InComponent->bRayTracingFarField;
	bHoldout = InComponent->bHoldout;
	bWantsEditorEffects = InComponent->bWantsEditorEffects;
	bIsFirstPerson = InComponent->FirstPersonPrimitiveType == EFirstPersonPrimitiveType::FirstPerson;
	bIsFirstPersonWorldSpaceRepresentation = InComponent->FirstPersonPrimitiveType == EFirstPersonPrimitiveType::WorldSpaceRepresentation;
	bLumenHeightfield = InComponent->bLumenHeightfield;

	bIsVisible = InComponent->IsVisible();
	bIsVisibleEditor = InComponent->GetVisibleFlag();
	bSelected = InComponent->IsSelected();
	bIndividuallySelected = InComponent->IsComponentIndividuallySelected();
	bShouldRenderSelected = InComponent->ShouldRenderSelected();
	bCollisionEnabled = InComponent->IsCollisionEnabled();

	if (const AActor* ActorOwner = InComponent->GetOwner())
	{
		bIsHidden = ActorOwner->IsHidden();
#if WITH_EDITOR
		bIsHiddenEd = ActorOwner->IsHiddenEd();
		bIsOwnedByFoliage = FFoliageHelper::IsOwnedByFoliage(ActorOwner);
#endif

		if (bOnlyOwnerSee || bOwnerNoSee || bUseViewOwnerDepthPriorityGroup || bIsFirstPersonWorldSpaceRepresentation)
		{
			// Make a list of the actors which directly or indirectly own the InComponent.
			for (const AActor* CurrentOwner = ActorOwner; CurrentOwner; CurrentOwner = CurrentOwner->GetOwner())
			{
				ActorOwners.Add(CurrentOwner->GetUniqueID());
			}
			
			// Append "visibility owners" from the component
			UPrimitiveComponentUtilities::ForEachVisibilityOwner(InComponent, [this](TObjectPtr<const AActor> VisibilityOwner)
			{
				ActorOwners.Add(VisibilityOwner->GetUniqueID());
			});
		}

		bIsOwnerEditorOnly = InComponent->GetOwner()->IsEditorOnly();
	}
	bSupportsWorldPositionOffsetVelocity = InComponent->SupportsWorldPositionOffsetVelocity();
	bIsInstancedStaticMesh = Cast<UInstancedStaticMeshComponent>(InComponent) != nullptr;

	Mobility = InComponent->Mobility;;
	TranslucencySortPriority = InComponent->TranslucencySortPriority;
	TranslucencySortDistanceOffset = InComponent->TranslucencySortDistanceOffset;
	LightmapType = InComponent->GetLightmapType();
	ViewOwnerDepthPriorityGroup = InComponent->ViewOwnerDepthPriorityGroup;
	CustomDepthStencilValue = InComponent->CustomDepthStencilValue;
	CustomDepthStencilWriteMask = InComponent->CustomDepthStencilWriteMask;
	LightingChannels = InComponent->LightingChannels;
	RayTracingGroupCullingPriority = InComponent->RayTracingGroupCullingPriority;
	IndirectLightingCacheQuality = InComponent->IndirectLightingCacheQuality;
	ShadowCacheInvalidationBehavior = InComponent->ShadowCacheInvalidationBehavior;
	DepthPriorityGroup = InComponent->GetStaticDepthPriorityGroup();

	VirtualTextureLodBias = InComponent->VirtualTextureLodBias;
	VirtualTextureCullMips = InComponent->VirtualTextureCullMips;
	VirtualTextureMinCoverage = InComponent->VirtualTextureMinCoverage;
	ComponentId = InComponent->GetPrimitiveSceneId();
	VisibilityId = InComponent->VisibilityId;
	CachedMaxDrawDistance = InComponent->CachedMaxDrawDistance;
	MinDrawDistance = InComponent->MinDrawDistance;
	BoundsScale = InComponent->BoundsScale;
	RayTracingGroupId = InComponent->GetRayTracingGroupId();

	bHasStaticLighting = InComponent->HasStaticLighting();
	bHasValidSettingsForStaticLighting = InComponent->HasValidSettingsForStaticLighting(false);
	bIsPrecomputedLightingValid = InComponent->IsPrecomputedLightingValid();
	bShadowIndirectOnly = InComponent->GetShadowIndirectOnly();

	Component = const_cast<UPrimitiveComponent*>(InComponent);
	Owner = InComponent->GetOwner();

#if !WITH_STATE_STREAM
	World = InComponent->GetWorld();
#endif

	CustomPrimitiveData = &InComponent->GetCustomPrimitiveData();
	Scene = InComponent->GetScene();
	PrimitiveComponentInterface = InComponent->GetPrimitiveComponentInterface();

	if (Scene)
	{
		FeatureLevel = Scene->GetFeatureLevel();
	}
	else
	{
		FeatureLevel = ERHIFeatureLevel::Num;
	}

#if WITH_EDITOR
	HiddenEditorViews = InComponent->GetHiddenEditorViews();
	OverlayColor = InComponent->OverlayColor;
#endif
	
	AdditionalStatObjectPtr = InComponent->AdditionalStatObject();
	StatId = AdditionalStatObjectPtr ? AdditionalStatObjectPtr->GetStatID(true) : InComponent->GetStatID(true);

	TArray<URuntimeVirtualTexture*> const& VirtualTextures = InComponent->GetRuntimeVirtualTextures();
	RuntimeVirtualTextures = MakeArrayView(const_cast<URuntimeVirtualTexture**>(VirtualTextures.GetData()), VirtualTextures.Num());
	VirtualTextureRenderPassType = InComponent->GetVirtualTextureRenderPassType();
	VirtualTextureMainPassMaxDrawDistance = InComponent->GetVirtualTextureMainPassMaxDrawDistance();

	SetPSOPrecacheComponentData(InComponent->GetPSOPrecacheComponentData());

#if MESH_DRAW_COMMAND_STATS
	MeshDrawCommandStatsCategory = InComponent->GetMeshDrawCommandStatsCategory();
#endif
}

void FPrimitiveSceneProxyDesc::GetUsedMaterials(TArray<UMaterialInterface*>& OutMaterials, bool bGetDebugMaterials) const
{
	// Only UPrimitiveComponent should rely on this method 
	const UPrimitiveComponent* AsComponent = Cast<UPrimitiveComponent>(Component);
	check(AsComponent);

	return AsComponent->GetUsedMaterials(OutMaterials, bGetDebugMaterials);
}

void FPrimitiveSceneProxyDesc::SetPSOPrecacheComponentData(const FPSOPrecacheComponentData& PSOPrecacheComponentData)
{
#if UE_WITH_PSO_PRECACHING	
	bUsePSOPrecacheFallbackMaterial = PSOPrecacheComponentData.UsePSOPrecacheFallbackMaterial();
	check(!PSOPrecacheComponentData.UsePSOPrecacheFallbackMaterial() || !PSOPrecacheComponentData.GetMaterialPSOPrecacheRequestIDs().IsEmpty());
	if (ShouldBoostPSOPrecachePriorityOnDraw() && PSOPrecacheComponentData.IsPSOPrecaching())
	{
		MaterialPSOPrecacheRequestIDsToBoostOnDraw = PSOPrecacheComponentData.GetMaterialPSOPrecacheRequestIDs();
	}
#endif // UE_WITH_PSO_PRECACHING
}