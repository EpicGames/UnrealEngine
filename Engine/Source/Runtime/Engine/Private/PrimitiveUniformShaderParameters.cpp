// Copyright Epic Games, Inc. All Rights Reserved.

#include "PrimitiveUniformShaderParameters.h"
#include "PrimitiveSceneInfo.h"
#include "NaniteSceneProxy.h"
#include "ProfilingDebugging/LoadTimeTracker.h"
#include "SceneInterface.h"
#include "UnrealEngine.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "PrimitiveUniformShaderParametersBuilder.h"
#include "PrimitiveSceneShaderData.h"

FPrimitiveUniformShaderParametersBuilder& FPrimitiveUniformShaderParametersBuilder::InstanceDrawDistance(FVector2f DistanceMinMax)
{
	DistanceMinMax *= GetCachedScalabilityCVars().ViewDistanceScale;
	Parameters.InstanceDrawDistanceMinMaxSquared = FMath::Square(DistanceMinMax);
	bHasInstanceDrawDistanceCull = true;
	return *this;
}

FPrimitiveUniformShaderParametersBuilder& FPrimitiveUniformShaderParametersBuilder::InstanceWorldPositionOffsetDisableDistance(float WPODisableDistance)
{
	WPODisableDistance *= GetCachedScalabilityCVars().ViewDistanceScale;
	bHasWPODisableDistance = true;
	Parameters.InstanceWPODisableDistanceSquared = WPODisableDistance * WPODisableDistance;

	return *this;
}

FPrimitiveUniformShaderParametersBuilder& FPrimitiveUniformShaderParametersBuilder::PixelProgrammableDistance(float PixelProgrammableDistance)
{
	PixelProgrammableDistance *= GetCachedScalabilityCVars().ViewDistanceScale;
	Parameters.PixelProgrammableDistanceSquared = PixelProgrammableDistance * PixelProgrammableDistance;

	return *this;
}

FPrimitiveUniformShaderParametersBuilder& FPrimitiveUniformShaderParametersBuilder::NaniteAssemblyInfo(uint32 TransformOffset, uint32 TransformCount)
{
	check(TransformCount < NANITE_HIERARCHY_MAX_ASSEMBLY_TRANSFORMS);
	Parameters.NaniteAssemblyTransformOffset = TransformOffset;
	Parameters.NaniteAssemblyTransformCount = TransformCount;

	return *this;
}

void FSinglePrimitiveStructured::InitRHI(FRHICommandListBase& RHICmdList)
{
	SCOPED_LOADTIMER(FSinglePrimitiveStructuredBuffer_InitRHI);

	{
		const static FLazyName ClassName(TEXT("FSinglePrimitiveStructured"));
		const FRHITextureCreateDesc Desc =
			FRHITextureCreateDesc::Create2D(TEXT("PrimitiveSceneDataTexture"), FPrimitiveSceneShaderData::DataStrideInFloat4s, 1, PF_A32B32G32R32F)
			.SetFlags(ETextureCreateFlags::ShaderResource | ETextureCreateFlags::UAV)
			.SetClassName(ClassName);

		PrimitiveSceneDataTextureRHI = RHICmdList.CreateTexture(Desc);
		PrimitiveSceneDataTextureSRV = RHICmdList.CreateShaderResourceView(PrimitiveSceneDataTextureRHI, FRHIViewDesc::CreateTextureSRV().SetDimensionFromTexture(PrimitiveSceneDataTextureRHI));
	}

	{
		const FRHIBufferCreateDesc Desc =
			FRHIBufferCreateDesc::CreateStructured<FVector4f>(TEXT("SkyIrradianceEnvironmentMap"), 8)
			.AddUsage(BUF_Static | BUF_ShaderResource)
			.DetermineInitialState();
		SkyIrradianceEnvironmentMapRHI = RHICmdList.CreateBuffer(Desc);
		SkyIrradianceEnvironmentMapSRV = RHICmdList.CreateShaderResourceView(SkyIrradianceEnvironmentMapRHI, FRHIViewDesc::CreateBufferSRV().SetTypeFromBuffer(SkyIrradianceEnvironmentMapRHI));
	}

	UploadToGPU(RHICmdList);
}

void FSinglePrimitiveStructured::UploadToGPU(FRHICommandListBase& RHICmdList)
{
}

TGlobalResource<FSinglePrimitiveStructured> GIdentityPrimitiveBuffer;

FPrimitiveSceneShaderData::FPrimitiveSceneShaderData(const FPrimitiveSceneProxy* RESTRICT Proxy)
	: Data(InPlace, NoInit)
{
	FPrimitiveSceneShaderData::BuildDataFromProxy(Proxy, Data.GetData());
}

void FPrimitiveSceneShaderData::BuildDataFromProxy(const FPrimitiveSceneProxy* RESTRICT Proxy, FVector4f* RESTRICT OutData)
{
	FPrimitiveUniformShaderParametersBuilder Builder = FPrimitiveUniformShaderParametersBuilder{};
	Proxy->BuildUniformShaderParameters(Builder);
	Setup(Builder.Build(), OutData);
}

/**
 * Helper struct to make sure integers are converted to float as needed.
 */
struct FAsFloat
{
	FORCEINLINE FAsFloat(uint32 InValue) : FloatValue(FMath::AsFloat(InValue)) {}
	FORCEINLINE FAsFloat(float InValue) : FloatValue(InValue) {}

	float FloatValue;
};

FORCEINLINE void Store4(FVector4f *Data, int32 Offset, FAsFloat X, FAsFloat Y, FAsFloat Z, FAsFloat W)
{
	VectorRegister4f VR = MakeVectorRegisterFloat(X.FloatValue, Y.FloatValue, Z.FloatValue, W.FloatValue);
	VectorStoreAligned(VR, &Data[Offset].X);
}

FORCEINLINE void Store4(FVector4f *Data, int32 Offset, const FVector3f &XYZ, FAsFloat W)
{
	Store4(Data, Offset, XYZ.X, XYZ.Y, XYZ.Z, W.FloatValue);
}

FORCEINLINE void StoreTransposed(FVector4f *Data, int32 StartOffset, const FMatrix44f &Matrix)
{
	Store4(Data, StartOffset + 0, Matrix.M[0][0],   Matrix.M[1][0],   Matrix.M[2][0],   Matrix.M[3][0]);
	Store4(Data, StartOffset + 1, Matrix.M[0][1],   Matrix.M[1][1],   Matrix.M[2][1],   Matrix.M[3][1]);
	Store4(Data, StartOffset + 2, Matrix.M[0][2],   Matrix.M[1][2],   Matrix.M[2][2],   Matrix.M[3][2]);
}

void FPrimitiveSceneShaderData::Setup(const FPrimitiveUniformShaderParameters& PrimitiveUniformShaderParameters, FVector4f* RESTRICT OutData)
{
	static_assert(NUM_LIGHTING_CHANNELS == 3, "The FPrimitiveSceneShaderData packing currently assumes a maximum of 3 lighting channels.");

	// Note: layout must match GetPrimitiveData in usf

	Store4(OutData, 0,
		PrimitiveUniformShaderParameters.Flags,
		PrimitiveUniformShaderParameters.InstanceSceneDataOffset,
		PrimitiveUniformShaderParameters.NumInstanceSceneDataEntries,
		(uint32)PrimitiveUniformShaderParameters.SingleCaptureIndex | ((PrimitiveUniformShaderParameters.VisibilityFlags & 0xFFFFu) << 16u));

	Store4(OutData, 1,
		PrimitiveUniformShaderParameters.PositionHigh.X,
		PrimitiveUniformShaderParameters.PositionHigh.Y,
		PrimitiveUniformShaderParameters.PositionHigh.Z,
		PrimitiveUniformShaderParameters.PrimitiveComponentId);

	// Pack these matrices into the buffer as float3x4 transposed
	StoreTransposed(OutData, 2, PrimitiveUniformShaderParameters.LocalToRelativeWorld);
	StoreTransposed(OutData, 5, PrimitiveUniformShaderParameters.RelativeWorldToLocal);
	StoreTransposed(OutData, 8, PrimitiveUniformShaderParameters.PreviousLocalToRelativeWorld);
	StoreTransposed(OutData, 11, PrimitiveUniformShaderParameters.PreviousRelativeWorldToLocal);
	StoreTransposed(OutData, 14, PrimitiveUniformShaderParameters.WorldToPreviousWorld);


	OutData[17]	= FVector4f(PrimitiveUniformShaderParameters.InvNonUniformScale, PrimitiveUniformShaderParameters.ObjectBoundsX);
	OutData[18]	= PrimitiveUniformShaderParameters.ObjectWorldPositionHighAndRadius;
	OutData[19]	= FVector4f(PrimitiveUniformShaderParameters.ObjectWorldPositionLow, PrimitiveUniformShaderParameters.MinMaterialDisplacement);
	OutData[20]	= FVector4f(PrimitiveUniformShaderParameters.ActorWorldPositionHigh, PrimitiveUniformShaderParameters.MaxMaterialDisplacement);

	Store4(OutData, 21, PrimitiveUniformShaderParameters.ActorWorldPositionLow, PrimitiveUniformShaderParameters.LightmapUVIndex);
	Store4(OutData, 22, PrimitiveUniformShaderParameters.ObjectOrientation, PrimitiveUniformShaderParameters.LightmapDataIndex);

	OutData[23]	= PrimitiveUniformShaderParameters.NonUniformScale;

	Store4(OutData, 24, PrimitiveUniformShaderParameters.PreSkinnedLocalBoundsMin, PrimitiveUniformShaderParameters.NaniteResourceID);
	Store4(OutData, 25, PrimitiveUniformShaderParameters.PreSkinnedLocalBoundsMax, PrimitiveUniformShaderParameters.NaniteHierarchyOffset);

	OutData[26]	= FVector4f(PrimitiveUniformShaderParameters.LocalObjectBoundsMin, PrimitiveUniformShaderParameters.ObjectBoundsY);
	OutData[27]	= FVector4f(PrimitiveUniformShaderParameters.LocalObjectBoundsMax, PrimitiveUniformShaderParameters.ObjectBoundsZ);

	Store4(OutData, 28, PrimitiveUniformShaderParameters.InstanceLocalBoundsCenter, PrimitiveUniformShaderParameters.InstancePayloadDataOffset);
	Store4(OutData, 29, PrimitiveUniformShaderParameters.InstanceLocalBoundsExtent, (PrimitiveUniformShaderParameters.InstancePayloadDataStride & 0x00FFFFFFu) | (PrimitiveUniformShaderParameters.InstancePayloadExtensionSize << 24u));

	Store4(OutData, 30, 
		PrimitiveUniformShaderParameters.WireframeAndPrimitiveColor.X, 
		PrimitiveUniformShaderParameters.WireframeAndPrimitiveColor.Y, 
		PrimitiveUniformShaderParameters.PackedNaniteFlags, 
		0u /*unused*/ );

	Store4(OutData, 31, 
		PrimitiveUniformShaderParameters.InstanceDrawDistanceMinMaxSquared.X, 
		PrimitiveUniformShaderParameters.InstanceDrawDistanceMinMaxSquared.Y, 
		PrimitiveUniformShaderParameters.InstanceWPODisableDistanceSquared, 
		PrimitiveUniformShaderParameters.NaniteRayTracingDataOffset);

	Store4(OutData, 32,
		PrimitiveUniformShaderParameters.MaxWPOExtent,
		PrimitiveUniformShaderParameters.CustomStencilValueAndMask,
		PrimitiveUniformShaderParameters.PixelProgrammableDistanceSquared,
		PrimitiveUniformShaderParameters.MaterialDisplacementFadeOutSize);

	Store4(OutData, 33,
		PrimitiveUniformShaderParameters.MeshPaintTextureDescriptor.X,
		PrimitiveUniformShaderParameters.MeshPaintTextureDescriptor.Y,
		PrimitiveUniformShaderParameters.NaniteAssemblyTransformOffset,
		PrimitiveUniformShaderParameters.NaniteAssemblyTransformCount);

	Store4(OutData, 34,
		PrimitiveUniformShaderParameters.MaterialCacheDescriptor,
		// Note: AnimationMinScreenSize could be packed to fewer bits as it needs to represent values in the range 0-1, and special case -1.
		PrimitiveUniformShaderParameters.AnimationMinScreenSize,
		PrimitiveUniformShaderParameters.InstancingFadeOutParams.X,
		PrimitiveUniformShaderParameters.InstancingFadeOutParams.Y);

	// Set all the custom primitive data float4. This matches the loop in SceneData.ush
	const int32 CustomPrimitiveDataStartIndex = 35;
	for (int32 DataIndex = 0; DataIndex < FCustomPrimitiveData::NumCustomPrimitiveDataFloat4s; ++DataIndex)
	{
		OutData[CustomPrimitiveDataStartIndex + DataIndex] = PrimitiveUniformShaderParameters.CustomPrimitiveData[DataIndex];
	}
}

void FPrimitiveSceneShaderData::Setup(const FPrimitiveUniformShaderParameters& PrimitiveUniformShaderParameters)
{
	FPrimitiveSceneShaderData::Setup(PrimitiveUniformShaderParameters, Data.GetData());
}

TUniformBufferRef<FPrimitiveUniformShaderParameters> CreatePrimitiveUniformBufferImmediate(
	const FMatrix& LocalToWorld,
	const FBoxSphereBounds& WorldBounds,
	const FBoxSphereBounds& LocalBounds,
	const FBoxSphereBounds& PreSkinnedLocalBounds,
	bool bReceivesDecals,
	bool bOutputVelocity
)
{
	check(IsInRenderingThread());
	return TUniformBufferRef<FPrimitiveUniformShaderParameters>::CreateUniformBufferImmediate(
		FPrimitiveUniformShaderParametersBuilder{}
		.Defaults()
			.LocalToWorld(LocalToWorld)
			.ActorWorldPosition(WorldBounds.Origin)
			.WorldBounds(WorldBounds)
			.LocalBounds(LocalBounds)
			.PreSkinnedLocalBounds(PreSkinnedLocalBounds)
			.ReceivesDecals(bReceivesDecals)
			.OutputVelocity(bOutputVelocity)
		.Build(),
		UniformBuffer_MultiFrame
	);
}

FPrimitiveUniformShaderParameters GetIdentityPrimitiveParameters()
{
	// Don't use FMatrix44f::Identity here as GetIdentityPrimitiveParameters is used by TGlobalResource<FIdentityPrimitiveUniformBuffer> and because
	// static initialization order is undefined, FMatrix44f::Identity might be all 0's or random data the first time this is called.
	return FPrimitiveUniformShaderParametersBuilder{}
		.Defaults()
			.LocalToWorld(FMatrix(FPlane(1, 0, 0, 0), FPlane(0, 1, 0, 0), FPlane(0, 0, 1, 0), FPlane(0, 0, 0, 1)))
			.ActorWorldPosition(FVector(0.0, 0.0, 0.0))
			.WorldBounds(FBoxSphereBounds(EForceInit::ForceInit))
			.LocalBounds(FBoxSphereBounds(EForceInit::ForceInit))
		.Build();
}