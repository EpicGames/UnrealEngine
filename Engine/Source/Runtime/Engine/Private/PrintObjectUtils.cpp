// Copyright Epic Games, Inc. All Rights Reserved.

#include "PrintObjectUtils.h"

#if !UE_BUILD_SHIPPING

#include "Containers/Set.h"
#include "HAL/IConsoleManager.h"
#include "UObject/Object.h"
#include "UObject/OverridableManager.h"
#include "UObject/OverriddenPropertySet.h"
#include "UObject/Class.h"
#include "UObject/UnrealType.h"
#include "UObject/UObjectHash.h"
#include "Misc/StringBuilder.h"

#if WITH_EDITORONLY_DATA
#include "UObject/InstanceDataObjectUtils.h"
#include "UObject/PropertyBagRepository.h"
#include "UObject/PropertyStateTracking.h"
#endif

namespace UE
{

static void LogObjectState(const UObject* Object, FOutputDevice* OutputDevice, bool bVerbose = false, const TCHAR* Preamble = TEXT(""))
{
	TStringBuilder<32> NameBuilder;
	TStringBuilder<32> ClassNameBuilder;
	TStringBuilder<256> PathBuilder;

	if (Object)
	{
		Object->GetFName().ToString(NameBuilder);
		Object->GetPathName(/*StopOuter*/nullptr, PathBuilder);
	}
	else
	{
		NameBuilder.Append(TEXT("<NULL>"));
		PathBuilder.Append(TEXT(""));
	}

	const UClass* ObjectClass = Object ? Object->GetClass() : nullptr;
	if (ObjectClass)
	{
		ObjectClass->GetFName().ToString(ClassNameBuilder);
	}
	else
	{
		ClassNameBuilder.Append(TEXT("<NULL Class>"));
	}

	OutputDevice->Logf(TEXT("%s'%.*s' [%.*s] (%.*s)"), 
		Preamble, NameBuilder.Len(), NameBuilder.GetData(), ClassNameBuilder.Len(), ClassNameBuilder.GetData(), PathBuilder.Len(), PathBuilder.GetData());

	if (bVerbose)
	{
		EObjectFlags ObjectFlags = Object ? Object->GetFlags() : EObjectFlags::RF_NoFlags;
		OutputDevice->Logf(TEXT("\tFlags: %s"), *LexToString(ObjectFlags));

		OutputDevice->Logf(TEXT("\tAddress: 0x%p"), Object);
	}
}

void PrintObjectsInOuter(UObject* Object, EPrintObjectFlag Flags, FOutputDevice* InOutputDevice)
{
	FOutputDevice* OutputDevice = InOutputDevice ? InOutputDevice : GLog;
	if (OutputDevice)
	{
		if (Object)
		{
			TArray<UObject*> ChildObjects;
			GetObjectsWithOuter(Object, ChildObjects, EGetObjectsFlags::IncludeNestedObjects);

			bool bVerbose = EnumHasAnyFlags(Flags, EPrintObjectFlag::Verbose);

			FString Preamble = FString::Printf(TEXT("Printing %d object(s) under: "), ChildObjects.Num());
			LogObjectState(Object, OutputDevice, bVerbose, *Preamble);

			for (UObject* ChildObject : ChildObjects)
			{
				LogObjectState(ChildObject, OutputDevice, bVerbose);
			}
		}
		else
		{
			OutputDevice->Logf(TEXT("PrintObjectsInOuter: NULL object"));
		}
	}
}

void PrintObjectsWithName(const TCHAR* ObjectName, EPrintObjectFlag Flags, FOutputDevice* InOutputDevice)
{
	FOutputDevice* OutputDevice = InOutputDevice ? InOutputDevice : GLog;
	if (OutputDevice)
	{
		if (ObjectName)
		{
			TArray<UObject*> Objects;
			bool bDidFindAny = StaticFindAllObjectsSafe(Objects, UObject::StaticClass(), ObjectName);

			if (bDidFindAny)
			{
				OutputDevice->Logf(TEXT("Printing %d object(s) with name: '%s'"), Objects.Num(), ObjectName);

				bool bVerbose = EnumHasAnyFlags(Flags, EPrintObjectFlag::Verbose);

				for (UObject* Object : Objects)
				{
					LogObjectState(Object, OutputDevice, bVerbose);
				}
			}
			else
			{
				OutputDevice->Logf(TEXT("PrintObjectsWithName: failed to find any objects with name: '%s'"), ObjectName);
			}
		}
		else
		{
			OutputDevice->Logf(TEXT("PrintObjectsWithName: NULL object name"));
		}
	}
}

void PrintStructProperties(UStruct* Struct, void* StructData, EPrintObjectFlag Flags, FOutputDevice* InOutputDevice)
{
	FOutputDevice* OutputDevice = InOutputDevice ? InOutputDevice : GLog;
	if (!OutputDevice)
	{
		return;
	}

	if (!Struct)
	{
		OutputDevice->Logf(TEXT("PrintStructProperties: NULL struct"));
	}
	if (!StructData)
	{
		OutputDevice->Logf(TEXT("PrintStructProperties: NULL StructData"));
	}

	if (Struct && StructData)
	{
		OutputDevice->Logf(TEXT("Printing properties for struct of type: '%s'"), *Struct->GetName());

		// Keep track of visited property-owner pairs to avoid referencing cycles.
		TSet<TTuple<const FProperty*, void*>> VisitedPropOwners;

		bool bIncludeinitializationState = EnumHasAnyFlags(Flags, EPrintObjectFlag::PropertyInitializationState);

		Struct->Visit(StructData, [&VisitedPropOwners, bIncludeinitializationState, OutputDevice](const FPropertyVisitorContext& Context)->EPropertyVisitorControlFlow
		{
			const FPropertyVisitorPath& PropertyPath = Context.Path;
			const FPropertyVisitorData& Data = Context.Data;
			const FProperty* Property = PropertyPath.Top().Property;
			void* Owner = Data.ParentStructData;
			TTuple<const FProperty*, void*> PropOwner(Property, Owner);

			if (!Property || VisitedPropOwners.Contains(PropOwner))
			{
				return EPropertyVisitorControlFlow::StepOver;
			}

			const UStruct* OwnerType = PropertyPath.Top().ParentStructType;

			UObject* OwnerObject = nullptr;

			if (OwnerType && OwnerType->IsChildOf<UObject>())
			{
				OwnerObject = static_cast<UObject*>(Owner);
			}

			void* PropData = Property->ContainerPtrToValuePtr<void>(Owner);
			FString PropValueAsString;
			Property->ExportText_Direct(PropValueAsString, PropData, PropData, OwnerObject, PPF_None);

			FString OwnerPath = OwnerObject ? OwnerObject->GetPathName() : FString();

			FString InitializationStateText;

			if (bIncludeinitializationState)
			{
				#if WITH_EDITORONLY_DATA
				UE::FInitializedPropertyValueState InitializedState(OwnerType, Owner);

				if (InitializedState.IsTracking())
				{
					bool bIsInitialized = !Property->HasAnyPropertyFlags(CPF_RequiredParm) || InitializedState.IsSet(Property);

					if (bIsInitialized)
					{
						InitializationStateText = TEXT(", (initialized)");
					}
					else
					{
						InitializationStateText = TEXT(", (uninitialized)");
					}
				}
				#endif
			}

			OutputDevice->Logf(TEXT("%s.%s: [%s] %s%s"), *OwnerPath, *Property->GetName(), *Property->GetClass()->GetName(), *PropValueAsString, *InitializationStateText);

			VisitedPropOwners.Add(PropOwner);

			return EPropertyVisitorControlFlow::StepOver;
		});
	}
}

void PrintObjectProperties(UObject* Object, EPrintObjectFlag Flags, FOutputDevice* InOutputDevice)
{
	FOutputDevice* OutputDevice = InOutputDevice ? InOutputDevice : GLog;
	if (!OutputDevice)
	{
		return;
	}

	if (Object)
	{
		bool bVerbose = EnumHasAnyFlags(Flags, EPrintObjectFlag::Verbose);
		bool bIncludeInitializationState = EnumHasAnyFlags(Flags, EPrintObjectFlag::PropertyInitializationState);
		bool bIncludePropertyAddresses = bVerbose;
			
		LogObjectState(Object, OutputDevice, bVerbose, TEXT("Printing properties of object "));

		// Keep track of visited property-owner pairs to avoid referencing cycles.
		TSet<TTuple<const FProperty*, void*>> VisitedPropOwners;

		FString ObjectPath = Object->GetPathName();

		Object->GetClass()->Visit(Object, [&VisitedPropOwners, Object, &ObjectPath, bIncludeInitializationState, bIncludePropertyAddresses, OutputDevice](const FPropertyVisitorContext& Context)->EPropertyVisitorControlFlow
		{
			const FPropertyVisitorPath& PropertyPath = Context.Path;
			const FPropertyVisitorData& Data = Context.Data;
			const FProperty* Property = PropertyPath.Top().Property;
			void* Owner = Data.ParentStructData;
			TTuple<const FProperty*, void*> PropOwner(Property, Owner);

			if (!Property || VisitedPropOwners.Contains(PropOwner))
			{
				return EPropertyVisitorControlFlow::StepOver;
			}

			const UStruct* OwnerType = PropertyPath.Top().ParentStructType;

			bool bIsInRootObject = true;

			if (OwnerType && OwnerType->IsChildOf<UObject>())
			{
				UObject* OwnerObject = static_cast<UObject*>(Owner);

				if (OwnerObject)
				{
					void* PropData = Property->ContainerPtrToValuePtr<void>(OwnerObject);
					FString PropValueAsString;
					Property->ExportText_Direct(PropValueAsString, PropData, PropData, OwnerObject, PPF_None);

					FString OwnerPath = OwnerObject->GetPathName();
					FString OwnerRelPath = OwnerPath.Contains(ObjectPath) ? OwnerPath.RightChop(ObjectPath.Len()) : OwnerPath;

					FString InitializationStateText;
					if (bIncludeInitializationState)
					{
						#if WITH_EDITORONLY_DATA
						UE::FInitializedPropertyValueState InitializedState(OwnerObject);

						if (InitializedState.IsTracking())
						{
							bool bIsInitialized = !Property->HasAnyPropertyFlags(CPF_RequiredParm) || InitializedState.IsSet(Property);

							if (bIsInitialized)
							{
								InitializationStateText = TEXT(", (initialized)");
							}
							else
							{
								InitializationStateText = TEXT(", (uninitialized)");
							}
						}
						#endif
					}

					FString PropAddrText;
					if (bIncludePropertyAddresses)
					{
						PropAddrText = FString::Printf(TEXT(", (0x%p)"), PropData);
					}
					
					OutputDevice->Logf(TEXT("%s.%s: [%s] %s%s%s"), 
						*OwnerRelPath, *Property->GetName(), *Property->GetClass()->GetName(), *PropValueAsString, *InitializationStateText, *PropAddrText);

					bIsInRootObject = OwnerObject->IsInOuter(Object);
				}
			}

			VisitedPropOwners.Add(PropOwner);

			if (bIsInRootObject)
			{
				return EPropertyVisitorControlFlow::StepInto;
			}
			else
			{
				// Don't step into external object references.
				return EPropertyVisitorControlFlow::StepOver;
			}
		});
	}
	else
	{
		OutputDevice->Logf(TEXT("PrintObjectProperties: NULL object"));
	}
}

void PrintObjectOverrides(UObject* Object, EPrintObjectFlag Flags, FOutputDevice* InOutputDevice)
{
	FOutputDevice* OutputDevice = InOutputDevice ? InOutputDevice : GLog;
	if (!OutputDevice)
	{
		return;
	}

	if (Object)
	{
		bool bVerbose = EnumHasAnyFlags(Flags, EPrintObjectFlag::Verbose);

		LogObjectState(Object, OutputDevice, bVerbose, TEXT("Printing overrides for object "));

		FOverridableManager& OverridableManager = FOverridableManager::Get();

		const FOverriddenPropertySet* Overrides = OverridableManager.GetOverriddenProperties(Object);

		if (Overrides)
		{
			FString OverrideStr = Overrides->ToDebugString();
			OutputDevice->Logf(TEXT("%s"), *OverrideStr);
		}
		else
		{
			OutputDevice->Logf(TEXT("No overrides found for object: %s"), *Object->GetPathName());
		}
	}
	else
	{
		OutputDevice->Logf(TEXT("PrintObjectOverrides: NULL object"));
	}
}

void PrintObjectArchetype(UObject* Object, EPrintObjectFlag Flags, FOutputDevice* InOutputDevice)
{
	FOutputDevice* OutputDevice = InOutputDevice ? InOutputDevice : GLog;
	if (OutputDevice)
	{
		if (Object)
		{
			UObject* Archetype = Object->GetArchetype();

			bool bVerbose = EnumHasAnyFlags(Flags, EPrintObjectFlag::Verbose);
			bool bShowFullChain = EnumHasAnyFlags(Flags, EPrintObjectFlag::FullArchetypeChain);

			if (!bShowFullChain)
			{
				LogObjectState(Object, OutputDevice, bVerbose, TEXT("Printing archetype for object: "));
				LogObjectState(Archetype, OutputDevice, bVerbose);
			}
			else
			{
				LogObjectState(Object, OutputDevice, bVerbose, TEXT("Printing archetype chain for object: "));

				while (Archetype != nullptr)
				{
					LogObjectState(Archetype, OutputDevice, bVerbose);

					Archetype = Archetype->GetArchetype();
				}
			}
		}
		else
		{
			OutputDevice->Logf(TEXT("PrintObjectArchetype: NULL object"));
		}
	}
}

void PrintObjectIDO(UObject* Object, EPrintObjectFlag Flags, FOutputDevice* InOutputDevice)
{
	FOutputDevice* OutputDevice = InOutputDevice ? InOutputDevice : GLog;
	if (OutputDevice)
	{
		#if WITH_EDITORONLY_DATA
		if (Object)
		{
			FString ObjectPath = Object->GetPathName();

			UObject* IDO = UE::FPropertyBagRepository::Get().FindInstanceDataObject(Object);

			bool bVerbose = EnumHasAnyFlags(Flags, EPrintObjectFlag::Verbose);

			if (IDO)
			{
				UObject* Archetype = Object->GetArchetype();

				LogObjectState(Object, OutputDevice, bVerbose, TEXT("Printing IDO (Instance Data Object) for object: "));
				LogObjectState(IDO, OutputDevice, bVerbose);
			}
			else
			{
				LogObjectState(Object, OutputDevice, bVerbose, TEXT("PrintObjectIDO: No IDO (Instance Data Object) found for object: "));
			}
		}
		else
		{
			OutputDevice->Logf(TEXT("PrintObjectIDO: NULL object"));
		}
		#else
		OutputDevice->Logf(TEXT("PrintObjectIDO: IDOs (Instance Data Objects) not supported in the current build"));
		#endif
	}
}

void PrintClassDefaultObject(const UClass* Class, EPrintObjectFlag Flags, FOutputDevice* InOutputDevice)
{
	FOutputDevice* OutputDevice = InOutputDevice ? InOutputDevice : GLog;
	if (OutputDevice)
	{
		if (Class)
		{
			UObject* CDO = Class->GetDefaultObject(/*bCreateIfNeeded*/false);

			bool bVerbose = EnumHasAnyFlags(Flags, EPrintObjectFlag::Verbose);

			if (CDO)
			{
				LogObjectState(Class, OutputDevice, bVerbose, TEXT("Printing Class Default Object for class: "));
				LogObjectState(CDO, OutputDevice, bVerbose);
			}
			else
			{
				LogObjectState(Class, OutputDevice, bVerbose, TEXT("PrintClassDefaultObject: No Class Default Object found for class: "));
			}
		}
		else
		{
			OutputDevice->Logf(TEXT("PrintClassDefaultObject: NULL class"));
		}
	}
}

void PrintStructHierarchy(const UStruct* Struct, EPrintObjectFlag Flags, FOutputDevice* InOutputDevice)
{
	FOutputDevice* OutputDevice = InOutputDevice ? InOutputDevice : GLog;
	if (OutputDevice)
	{
		if (Struct)
		{
			bool bVerbose = EnumHasAnyFlags(Flags, EPrintObjectFlag::Verbose);

			const UStruct* CurrentStruct = Struct;

			TStringBuilder<16> PaddingBuilder;

			while (CurrentStruct)
			{
				LogObjectState(CurrentStruct, OutputDevice, bVerbose, *PaddingBuilder);

				CurrentStruct = CurrentStruct->GetSuperStruct();

				PaddingBuilder.Append(TEXT("  "));
			}
		}
		else
		{
			OutputDevice->Logf(TEXT("PrintStructHierarchy: NULL struct"));
		}
	}
}

static void ParseObjectIdAndFlags(const TArray<FString>& Args, FString& OutObjectIdentifier, EPrintObjectFlag& OutFlags)
{
	OutFlags = EPrintObjectFlag::None;

	for (FString Arg : Args)
	{
		if (Arg.Equals(TEXT("Verbose=true"), ESearchCase::IgnoreCase))
		{
			EnumAddFlags(OutFlags, EPrintObjectFlag::Verbose);
		}
		else if (Arg.Equals(TEXT("InitState=true"), ESearchCase::IgnoreCase))
		{
			EnumAddFlags(OutFlags, EPrintObjectFlag::PropertyInitializationState);
		}
		else if (Arg.Equals(TEXT("ArchetypeChain=true"), ESearchCase::IgnoreCase))
		{
			EnumAddFlags(OutFlags, EPrintObjectFlag::FullArchetypeChain);
		}
		else
		{
			OutObjectIdentifier = Arg;
		}
	}
}

static void LogHelp(FOutputDevice& OutputDevice)
{
	OutputDevice.Logf(TEXT("Optional flags:"));
	OutputDevice.Logf(TEXT("Verbose=true - Include verbose information"));
	OutputDevice.Logf(TEXT("InitState=true - Include the initialization state for properties (only relevant for functions that print properties)."));
	OutputDevice.Logf(TEXT("ArchetypeChain=true - Show the full archetype hierarchy (only relevant for functions that print archetypes)."));
}


FAutoConsoleCommandWithArgsAndOutputDevice CVarCommandPrintObjectsInOuter(
	TEXT("Obj.DumpObjectsInOuter"),
	TEXT("Lists all objects under a specified parent (the parent object must be specified as a name or path, e.g., /MyLevel/MyLevel.MyLevel:PersistentLevel)."),
	FConsoleCommandWithArgsAndOutputDeviceDelegate::CreateLambda([](const TArray<FString>& Args, FOutputDevice& OutputDevice)
	{
		if (Args.Num() > 0)
		{
			FString ObjectPath;
			EPrintObjectFlag Flags;
			ParseObjectIdAndFlags(Args, ObjectPath, Flags);

			UObject* Object = FindFirstObjectSafe<UObject>(*ObjectPath);

			if (Object)
			{
				PrintObjectsInOuter(Object, Flags, &OutputDevice);
			}
			else
			{
				OutputDevice.Logf(TEXT("PrintObjectsInOuter: failed to find any objects for path: '%s'"), *ObjectPath);
			}
		}
		else
		{
			OutputDevice.Logf(TEXT("PrintObjectsInOuter: no object path specified (example usage: Obj.DumpObjectsInOuter /MyLevel/MyLevel.MyLevel:PersistentLevel)"));
			LogHelp(OutputDevice);
		}
	}));

FAutoConsoleCommandWithArgsAndOutputDevice CVarCommandPrintObjectsWithName(
	TEXT("Obj.DumpObjectsWithName"),
	TEXT("Lists all objects with a given name."),
	FConsoleCommandWithArgsAndOutputDeviceDelegate::CreateLambda([](const TArray<FString>& Args, FOutputDevice& OutputDevice)
	{
		if (Args.Num() > 0)
		{
			FString ObjectName;
			EPrintObjectFlag Flags;
			ParseObjectIdAndFlags(Args, ObjectName, Flags);

			PrintObjectsWithName(*ObjectName, Flags, &OutputDevice);
		}
		else
		{
			OutputDevice.Logf(TEXT("PrintObjectsWithName: no object name specified (example usage: Obj.DumpObjectsWithName PersistentLevel)"));
			LogHelp(OutputDevice);
		}
	}));

FAutoConsoleCommandWithArgsAndOutputDevice CVarCommandPrintObjectProperties(
	TEXT("Obj.DumpProperties"),
	TEXT("Lists the properties of an object (the object must be specified as a name or path, e.g., /MyLevel/MyLevel.MyLevel:PersistentLevel.MyActor). Pass InitState=true to iclude the properties' initialization state."),
	FConsoleCommandWithArgsAndOutputDeviceDelegate::CreateLambda([](const TArray<FString>& Args, FOutputDevice& OutputDevice)
	{
		if (Args.Num() > 0)
		{
			FString ObjectPath;
			EPrintObjectFlag Flags;
			ParseObjectIdAndFlags(Args, ObjectPath, Flags);

			UObject* Object = FindFirstObjectSafe<UObject>(*ObjectPath);

			if (Object)
			{
				PrintObjectProperties(Object, Flags, &OutputDevice);
			}
			else
			{
				OutputDevice.Logf(TEXT("PrintObjectProperties: failed to find any objects for path: '%s'"), *ObjectPath);
			}
		}
		else
		{
			OutputDevice.Logf(TEXT("PrintObjectProperties: no object path specified (example usage: Obj.DumpProperties /MyLevel/MyLevel.MyLevel:PersistentLevel.MyActor)"));
			LogHelp(OutputDevice);
		}
	}));

FAutoConsoleCommandWithArgsAndOutputDevice CVarCommandPrintObjectOverrides(
	TEXT("Obj.DumpOverrides"),
	TEXT("Lists the overridable serialization overrides of an object (the object must be specified as a name or path, e.g., /MyLevel/MyLevel.MyLevel:PersistentLevel.MyActor)."),
	FConsoleCommandWithArgsAndOutputDeviceDelegate::CreateLambda([](const TArray<FString>& Args, FOutputDevice& OutputDevice)
	{
		if (Args.Num() > 0)
		{
			FString ObjectPath;
			EPrintObjectFlag Flags;
			ParseObjectIdAndFlags(Args, ObjectPath, Flags);

			UObject* Object = FindFirstObjectSafe<UObject>(*ObjectPath);

			if (Object)
			{
				PrintObjectOverrides(Object, Flags, &OutputDevice);
			}
			else
			{
				OutputDevice.Logf(TEXT("PrintObjectOverrides: failed to find any objects for path: '%s'"), *ObjectPath);
			}
		}
		else
		{
			OutputDevice.Logf(TEXT("PrintObjectOverrides: no object path specified (example usage: Obj.DumpOverrides /MyLevel/MyLevel.MyLevel:PersistentLevel.MyActor)"));
			LogHelp(OutputDevice);
		}
	}));

FAutoConsoleCommandWithArgsAndOutputDevice CVarCommandPrintObjectArchetype(
	TEXT("Obj.DumpArchetype"),
	TEXT("Outputs an object's archetype (the object must be specified as a name or path, e.g., /MyLevel/MyLevel.MyLevel:PersistentLevel.MyActor)."),
	FConsoleCommandWithArgsAndOutputDeviceDelegate::CreateLambda([](const TArray<FString>& Args, FOutputDevice& OutputDevice)
	{
		if (Args.Num() > 0)
		{
			FString ObjectPath;
			EPrintObjectFlag Flags;
			ParseObjectIdAndFlags(Args, ObjectPath, Flags);

			UObject* Object = FindFirstObjectSafe<UObject>(*ObjectPath);

			if (Object)
			{
				PrintObjectArchetype(Object, Flags, &OutputDevice);
			}
			else
			{
				OutputDevice.Logf(TEXT("PrintObjectArchetype: failed to find any objects for path: '%s'"), *ObjectPath);
			}
		}
		else
		{
			OutputDevice.Logf(TEXT("PrintObjectArchetype: no object path specified (example usage: Obj.DumpArchetype /MyLevel/MyLevel.MyLevel:PersistentLevel.MyActor)"));
			LogHelp(OutputDevice);
		}
	}));

FAutoConsoleCommandWithArgsAndOutputDevice CVarCommandPrintObjectIDO(
	TEXT("Obj.DumpIDO"),
	TEXT("Outputs an object's IDO (Instance Data Object) (the object must be specified as a name or path, e.g., /MyLevel/MyLevel.MyLevel:PersistentLevel.MyActor)."),
	FConsoleCommandWithArgsAndOutputDeviceDelegate::CreateLambda([](const TArray<FString>& Args, FOutputDevice& OutputDevice)
	{
		if (Args.Num() > 0)
		{
			FString ObjectPath;
			EPrintObjectFlag Flags;
			ParseObjectIdAndFlags(Args, ObjectPath, Flags);
			
			UObject* Object = FindFirstObjectSafe<UObject>(*ObjectPath);

			if (Object)
			{
				PrintObjectIDO(Object, Flags, &OutputDevice);
			}
			else
			{
				OutputDevice.Logf(TEXT("PrintObjectIDO: failed to find any objects for path: '%s'"), *ObjectPath);
			}
		}
		else
		{
			OutputDevice.Logf(TEXT("PrintObjectIDO: no object path specified (example usage: Obj.DumpArchetype /MyLevel/MyLevel.MyLevel:PersistentLevel.MyActor)"));
			LogHelp(OutputDevice);
		}
	}));

FAutoConsoleCommandWithArgsAndOutputDevice CVarCommandPrintCDO(
	TEXT("Obj.DumpCDO"),
	TEXT("Outputs a class' Class Default Object."),
	FConsoleCommandWithArgsAndOutputDeviceDelegate::CreateLambda([](const TArray<FString>& Args, FOutputDevice& OutputDevice)
	{
		if (Args.Num() > 0)
		{
			FString ClassPath;
			EPrintObjectFlag Flags;
			ParseObjectIdAndFlags(Args, ClassPath, Flags);

			UClass* Class = FindFirstObjectSafe<UClass>(*ClassPath);

			if (Class)
			{
				PrintClassDefaultObject(Class, Flags, &OutputDevice);
			}
			else
			{
				OutputDevice.Logf(TEXT("PrintClassDefaultObject: failed to find any classes for path: '%s'"), *ClassPath);
			}
		}
		else
		{
			OutputDevice.Logf(TEXT("PrintClassDefaultObject: no class name or path specified (example usage: Obj.DumpCDO MyActor)"));
			LogHelp(OutputDevice);
		}
	}));

FAutoConsoleCommandWithArgsAndOutputDevice CVarCommandPrintStructHierarchy(
	TEXT("Obj.DumpStructHierarchy"),
	TEXT("Outputs a struct (or class) type hierarchy."),
	FConsoleCommandWithArgsAndOutputDeviceDelegate::CreateLambda([](const TArray<FString>& Args, FOutputDevice& OutputDevice)
	{
		if (Args.Num() > 0)
		{
			FString StructPath;
			EPrintObjectFlag Flags;
			ParseObjectIdAndFlags(Args, StructPath, Flags);

			UStruct* Struct = FindFirstObjectSafe<UStruct>(*StructPath);

			if (Struct)
			{
				PrintStructHierarchy(Struct, Flags, &OutputDevice);
			}
			else
			{
				OutputDevice.Logf(TEXT("PrintStructHierarchy: failed to find any types for path: '%s'"), *StructPath);
			}
		}
		else
		{
			OutputDevice.Logf(TEXT("PrintStructHierarchy: no struct name or path specified (example usage: Obj.DumpStructHierarchy MyActor)"));
			LogHelp(OutputDevice);
		}
	}));
	
} // namespace UE

#endif // !UE_BUILD_SHIPPING
