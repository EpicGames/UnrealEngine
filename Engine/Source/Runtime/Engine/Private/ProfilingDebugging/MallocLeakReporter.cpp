// Copyright Epic Games, Inc. All Rights Reserved.

#include "ProfilingDebugging/MallocLeakReporter.h"
#include "Engine/Engine.h"
#include "Misc/Paths.h"
#include "Engine/World.h"


DEFINE_LOG_CATEGORY_STATIC(LogLeakDetector, Log, All);

FMallocLeakReporter& FMallocLeakReporter::Get()
{
	static FMallocLeakReporter Singleton;
	return Singleton;
}

FMallocLeakReporter::FMallocLeakReporter()
	: Enabled(false)
	, ReportCount(0)
{
	// Default options for leak report
	DefaultLeakReportOptions.OnlyNonDeleters = true;
	DefaultLeakReportOptions.RateFilter = 0.1f;
	DefaultLeakReportOptions.SizeFilter = 512 * 1024;
	DefaultLeakReportOptions.SortBy = FMallocLeakReportOptions::ESortOption::SortRate;

	// default options for alloc report
	DefaultAllocReportOptions.SizeFilter = 1024 * 1024;
	DefaultAllocReportOptions.SortBy = FMallocLeakReportOptions::ESortOption::SortSize;
}

void FMallocLeakReporter::Start(int32 FilterSize/*=0*/, float ReportOnTime /*= 0.0f*/)
{
	// assume they want to change options so stop/start
	if (Enabled)
	{
		Stop();
	}

#if !MALLOC_LEAKDETECTION || PLATFORM_USES_FIXED_GMalloc_CLASS

	UE_LOGF(LogLeakDetector, Error, "Cannot track leaks, MALLOC_LEAKDETECTION=%d, PLATFORM_USES_FIXED_GMalloc_CLASS=%d (should be set as 1 & 0 in your Game<Config>Target.cs file)",
		MALLOC_LEAKDETECTION, PLATFORM_USES_FIXED_GMalloc_CLASS);

#else
	UE_LOGF(LogLeakDetector, Log, "Started Tracking Allocations > %d KiB", FilterSize / 1024);
	FMallocLeakDetection::Get().SetAllocationCollection(true, FilterSize);

	// Create a ticker to issue checkpoints
	CheckpointTicker = FTSTicker::GetCoreTicker().AddTicker(FTickerDelegate::CreateLambda([this](const float TimeDelta)
	{
		Checkpoint();
		return Enabled;
	}), 120.0f);

	// If specified, create a handler to generate reports periodically
	if (ReportOnTime > 0.0f)
	{
		ReportTicker = FTSTicker::GetCoreTicker().AddTicker(FTickerDelegate::CreateLambda([this](const float TimeDelta)
		{
			WriteReports();
			return true;
		}), ReportOnTime);
	}

	Enabled = true;
#endif
}

void FMallocLeakReporter::Stop()
{
	if (!Enabled)
	{
		return;
	}

#if MALLOC_LEAKDETECTION
	UE_LOGF(LogLeakDetector, Log, "Stopped tracking allocations");
	FMallocLeakDetection::Get().SetAllocationCollection(false);

	FTSTicker::RemoveTicker(CheckpointTicker);
	FTSTicker::RemoveTicker(ReportTicker);

	CheckpointTicker.Reset();
	ReportTicker.Reset();

	Enabled = false;
#endif // MALLOC_LEAKDETECTION
}

void FMallocLeakReporter::Clear()
{
#if MALLOC_LEAKDETECTION
	FMallocLeakDetection::Get().ClearData();
#endif
}

void FMallocLeakReporter::Checkpoint()
{
    QUICK_SCOPE_CYCLE_COUNTER(STAT_FMallocLeakDetection_Checkpoint);

#if MALLOC_LEAKDETECTION
	FMallocLeakDetection::Get().CheckpointLinearFit();
#endif
}

int32 FMallocLeakReporter::WriteReports(const uint32 ReportFlags /*=EReportOption::ReportAll*/)
{
    QUICK_SCOPE_CYCLE_COUNTER(STAT_FMallocLeakReporter_WriteReports);

	FString MapName = FPaths::GetBaseFilename(GWorld->GetName());

	FString BaseName = FString::Printf(TEXT("%03d_%s"), ReportCount++, *MapName);
	FString LeakName = BaseName + TEXT("_Leaks.txt");
	FString AllocName = BaseName + TEXT("_Allocs.txt");

	// write out leaks
	int32 LeakCount = WriteReport(*LeakName, DefaultLeakReportOptions);

	if (LeakCount > 0)
	{
		UE_LOGF(LogLeakDetector, Log, "Found %d leaks, report written to %ls", LeakCount, *LeakName);

		ReportDelegate.Broadcast(ReportCount, LeakCount);
	}
	else
	{
		UE_LOGF(LogLeakDetector, Log, "No leaks found");
	}

	// write out allocations
	if (ReportFlags & EReportOption::ReportAllocs)
	{
		WriteReport(*AllocName, DefaultAllocReportOptions);
	}

	// write out memreport
#if !UE_BUILD_SHIPPING
	if (ReportFlags & EReportOption::ReportMemReport)
	{
		FString Args = FString::Printf(TEXT(" -full -name=%s"), *BaseName);
		GEngine->HandleMemReportCommand(*Args, *GLog, GWorld);
	}
#endif

	return LeakCount;
}

int32 FMallocLeakReporter::WriteReport(const TCHAR* ReportName, const FMallocLeakReportOptions& Options)
{
#if MALLOC_LEAKDETECTION
	return FMallocLeakDetection::Get().DumpOpenCallstacks(ReportName, Options);
#else
	UE_LOGF(LogLeakDetector, Log, "Cannot report leaks. MALLOC_LEAKDETECTION=0");
	return 0;
#endif // MALLOC_LEAKDETECTION
}

/** 
	CLI interface for leak tracker.
*/

static FAutoConsoleCommand LeakReporterStartCommand(
	TEXT("mallocleak.start"),
	TEXT("Starts tracking allocations. Args -report=[secs] -size=[filter (KiB)]"),
	FConsoleCommandWithArgsDelegate::CreateLambda([](const TArray<FString>& Args) {
		FString ArgString = FString::Join(Args, TEXT(" "));
		int32 Size = 0;
		float ReportTime = 0;
		FParse::Value(*ArgString, TEXT("size="), Size); // KiB
		FParse::Value(*ArgString, TEXT("report="), ReportTime); // Seconds

		// FMallocLeakReporter expects FilterSize in bytes
		FMallocLeakReporter::Get().Start(Size * 1024, ReportTime);

		UE_LOGF(LogConsoleResponse, Display, "Tracking allocations >= %d KiB and reporting every %f seconds", Size, ReportTime);
	})
);

static FAutoConsoleCommand LeakReporterStopCommand(
	TEXT("mallocleak.stop"),
	TEXT("Stops tracking allocations"),
	FConsoleCommandDelegate::CreateLambda([]() {
		FMallocLeakReporter::Get().Stop();
		UE_LOGF(LogConsoleResponse, Display, "Stopped tracking allocations.");
	})
);

static FAutoConsoleCommand LeakReporterClearCommand(
	TEXT("mallocleak.clear"),
		TEXT("Clears recorded allocation info"),
		FConsoleCommandDelegate::CreateLambda([]() {
		FMallocLeakReporter::Get().Clear();
		UE_LOGF(LogConsoleResponse, Display, "Cleared recorded data.");
	})
);

static FAutoConsoleCommand LeakReporterReportCommand(
	TEXT("mallocleak.report"),
	TEXT("Writes malloc leak reports "),
	FConsoleCommandDelegate::CreateLambda([]() {
		FMallocLeakReporter::Get().WriteReports();
		UE_LOGF(LogConsoleResponse, Display, "Wrote out memory reports");
	})
);
