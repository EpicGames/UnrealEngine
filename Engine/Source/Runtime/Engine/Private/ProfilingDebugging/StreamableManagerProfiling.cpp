// Copyright Epic Games, Inc. All Rights Reserved.

#include "ProfilingDebugging/StreamableManagerProfiling.h"
#include "ProfilingDebugging/CountersTrace.h"
#include "ProfilingDebugging/CsvProfiler.h"
#include "ProfilingDebugging/ScopedTimers.h"
#include "Misc/CoreDelegates.h"
#include "UObject/UObjectGlobals.h"
#include "HAL/IConsoleManager.h"

CSV_DEFINE_CATEGORY_MODULE(ENGINE_API, StreamableManager, true);

#if UE_STREAMABLE_MANAGER_PROFILING_ENABLED

static bool GStreamableManagerProfilingEnabled = !UE_BUILD_SHIPPING;
static FAutoConsoleVariableRef CVarStreamableManagerProfilingEnabled(
#if UE_BUILD_SHIPPING
	TEXT("StreamableManager.Profiling.Enabled.Shipping"),
#else
	TEXT("StreamableManager.Profiling.Enabled"),
#endif
	GStreamableManagerProfilingEnabled,
	TEXT("Whether to enable the StreamableManager profiling."),
	FConsoleVariableDelegate::CreateLambda(
		[](IConsoleVariable* CVar)
		{
			FStreamableManagerProfiling::TryInitialize();
		}
	),
	ECVF_Default
);

TRACE_DECLARE_ATOMIC_INT_COUNTER(JITLoadersActive, TEXT("StreamableManager/JITLoadersActives"));
TRACE_DECLARE_ATOMIC_INT_COUNTER(StreamablesInFlight, TEXT("StreamableManager/StreamablesInFlight"));
TRACE_DECLARE_ATOMIC_INT_COUNTER(TotalStreamablesQueued, TEXT("StreamableManager/TotalStreamablesQueued"));
TRACE_DECLARE_ATOMIC_INT_COUNTER(TotalStreamablesCancelled, TEXT("StreamableManager/TotalStreamablesCancelled"));
TRACE_DECLARE_ATOMIC_FLOAT_COUNTER(AvgStreamableProcessingTime, TEXT("StreamableManager/AvgStreamableProcessingTime"));

std::atomic<int32> FStreamableManagerProfiling::JITLoadersActive = 0;
std::atomic<int32> FStreamableManagerProfiling::StreamablesInFlight = 0;
std::atomic<int32> FStreamableManagerProfiling::TotalStreamablesQueued = 0;
std::atomic<int32> FStreamableManagerProfiling::TotalStreamablesCancelled = 0;
std::atomic<double> FStreamableManagerProfiling::TotalStreamableProcessingTime = 0.0;
std::atomic<int32> FStreamableManagerProfiling::TotalStreamableProcessingTimeCount = 0;

FDelegateHandle FStreamableManagerProfiling::OnAsyncLoadingFlushedHandle;
FDelegateHandle FStreamableManagerProfiling::OnPreMapLoadHandle;
std::atomic<bool> FStreamableManagerProfiling::bIsInitialized = false;

void FStreamableManagerProfiling::TryInitialize()
{
	if (!GStreamableManagerProfilingEnabled)
	{
		if (bIsInitialized)
		{
			FCoreDelegates::OnAsyncLoadingFlushUpdate.Remove(OnAsyncLoadingFlushedHandle);
			FCoreUObjectDelegates::PreLoadMap.Remove(OnPreMapLoadHandle);
			OnAsyncLoadingFlushedHandle.Reset();
			OnPreMapLoadHandle.Reset();

			bIsInitialized = false;
			Reset();
		}
	}
	else
	{
		if (!bIsInitialized)
		{
			OnAsyncLoadingFlushedHandle = FCoreDelegates::OnAsyncLoadingFlushUpdate.AddStatic(&FStreamableManagerProfiling::OnAsyncLoadingFlushed);
			OnPreMapLoadHandle = FCoreUObjectDelegates::PreLoadMap.AddStatic(&FStreamableManagerProfiling::HandlePreLoadMap);

			Reset();
			bIsInitialized = true;
		}
	}
}

void FStreamableManagerProfiling::OnStreamableQueuedForAsyncLoad()
{
	if (bIsInitialized)
	{
		StreamablesInFlight.fetch_add(1, std::memory_order_relaxed);
		TotalStreamablesQueued.fetch_add(1, std::memory_order_relaxed);
	}	
}

void FStreamableManagerProfiling::OnStreamableCompleted(double Time)
{
	if (bIsInitialized)
	{
		check(FStreamableManagerProfiling::StreamablesInFlight);
		StreamablesInFlight.fetch_sub(1, std::memory_order_relaxed);
	
		AtomicDoubleFetchAdd(TotalStreamableProcessingTime, Time);
		TotalStreamableProcessingTimeCount.fetch_add(1, std::memory_order_relaxed);
	}
}

void FStreamableManagerProfiling::OnStreamablesCancelled(int32 NumCancelled)
{
	if (bIsInitialized)
	{
		TotalStreamablesCancelled.fetch_add(NumCancelled, std::memory_order_relaxed);
	}
}

void FStreamableManagerProfiling::OnJITLoaderCreated()
{
	if (bIsInitialized)
	{
		JITLoadersActive.fetch_add(1, std::memory_order_relaxed);
	}
}

void FStreamableManagerProfiling::OnJITLoaderDestroyed()
{
	if (bIsInitialized)
	{
		JITLoadersActive.fetch_sub(1, std::memory_order_relaxed);
	}
}

void FStreamableManagerProfiling::OnAsyncLoadingFlushed()
{
	check(IsInGameThread());
	check(bIsInitialized);

	const float AvgStreamableProcessingTime = TotalStreamableProcessingTime.load(std::memory_order_relaxed) / FMath::Max(TotalStreamableProcessingTimeCount.load(std::memory_order_relaxed), 1);

	TRACE_COUNTER_SET(JITLoadersActive, JITLoadersActive);
	TRACE_COUNTER_SET(StreamablesInFlight, StreamablesInFlight);
	TRACE_COUNTER_SET(TotalStreamablesQueued, TotalStreamablesQueued);
	TRACE_COUNTER_SET(TotalStreamablesCancelled, TotalStreamablesCancelled);
	TRACE_COUNTER_SET(AvgStreamableProcessingTime, AvgStreamableProcessingTime);

	CSV_CUSTOM_STAT(StreamableManager, JITLoadersActive, JITLoadersActive, ECsvCustomStatOp::Set);
	CSV_CUSTOM_STAT(StreamableManager, StreamablesInFlight, StreamablesInFlight, ECsvCustomStatOp::Set);
	CSV_CUSTOM_STAT(StreamableManager, TotalStreamablesQueued, TotalStreamablesQueued, ECsvCustomStatOp::Set);
	CSV_CUSTOM_STAT(StreamableManager, TotalStreamablesCancelled, TotalStreamablesCancelled, ECsvCustomStatOp::Set);
	CSV_CUSTOM_STAT(StreamableManager, AvgStreamableProcessingTime, AvgStreamableProcessingTime, ECsvCustomStatOp::Set);

	TotalStreamableProcessingTime = 0.0;
	TotalStreamableProcessingTimeCount = 0;
}

void FStreamableManagerProfiling::HandlePreLoadMap(const FString& MapName)
{
	check(IsInGameThread());
	check(bIsInitialized);

	// Reset total counters
	TotalStreamablesQueued = 0;
	TotalStreamablesCancelled = 0;
}

void FStreamableManagerProfiling::Reset()
{
	JITLoadersActive = 0;
	StreamablesInFlight = 0;
	TotalStreamablesQueued = 0;
	TotalStreamablesCancelled = 0;
	TotalStreamableProcessingTime = 0.0;
	TotalStreamableProcessingTimeCount = 0;
}

#endif // UE_STREAMABLE_MANAGER_PROFILING_ENABLED