// Copyright Epic Games, Inc. All Rights Reserved.

#include "Delegates/IDelegateInstance.h"
#include "Misc/App.h"
#include "Misc/CoreMisc.h"
#include "Misc/OutputDevice.h"
#include "Misc/Paths.h"
#include "HAL/FileManager.h"
#include "ProfilingDebugging/TraceAuxiliary.h"
#include "ProfilingDebugging/MiscTrace.h"
#include "Stats/StatsData.h"
#include "UnrealEngine.h"
#include "UnrealClient.h"

#if !defined(UE_SNAPSHOTHITCHES_ENABLED)
	#define UE_SNAPSHOTHITCHES_ENABLED !UE_BUILD_SHIPPING
#endif

#if UE_SNAPSHOTHITCHES_ENABLED && STATS

static class FTraceHitches : private FSelfRegisteringExec
{
private:
	virtual bool Exec_Runtime( UWorld*, const TCHAR* Cmd, FOutputDevice& Ar ) override
	{
		if (FParse::Command(&Cmd, TEXT("snapshothitches")))
		{
			HandleCommand(Cmd, Ar);
			return true;
		}
		return false;
	}

	void HandleCommand(const TCHAR* Cmd, FOutputDevice& Ar)
	{
#if 0
			bool bIsTest = FString(Cmd).Find(TEXT("-test")) != INDEX_NONE;
			if (bIsTest)
			{
				FPlatformProcess::Sleep(1.0f);
				return;
			}
#endif
			
			static bool bIsActive = false;
			bool bIsStart = FString(Cmd).Find(TEXT("-start")) != INDEX_NONE;
			bool bIsStop = FString(Cmd).Find(TEXT("-stop")) != INDEX_NONE;

			if (bIsStart && bIsActive)
				return;

			if (bIsStop && !bIsActive)
				return;

			FStatsThreadState& Stats = FStatsThreadState::GetLocalState();
			if (bIsStart)
			{
				// Disable screenshot channel, otherwise the last screenshot may end
				// up in the tail if two hitches are reported in close proximity
				FTraceAuxiliary::DisableChannels(TEXT("Screenshot"));
				SnapshotHitchesHandle = Stats.NewFrameDelegate.AddStatic(&FTraceHitches::SnapshotFrameOnHitch);
				UE_LOGF(LogEngine, Display, "Enabled automatic hitch snapshots");
				bIsActive = true;
			}
			else if (bIsStop)
			{
				Stats.NewFrameDelegate.Remove(SnapshotHitchesHandle);
				UE_LOGF(LogEngine, Display, "Disabled automatic hitch snapshots");
				bIsActive = false;
			}
	}

	static void SnapshotFrameOnHitch(int64 Frame)
	{
		// Avoid the hitch created by making the screenshot 
		// create a new hitch 
		static int64 LastHitchFrame = -ScreenshotFrameLag;
		if( LastHitchFrame + ScreenshotFrameLag > Frame )
		{
			return;
		}

		FStatsThreadState& Stats = FStatsThreadState::GetLocalState();

		const double GameThreadTime = FPlatformTime::ToSeconds((uint64)Stats.GetFastThreadFrameTime(Frame, EThreadType::Game));
		const double RenderThreadTime = FPlatformTime::ToSeconds((uint64)Stats.GetFastThreadFrameTime(Frame, EThreadType::Renderer));
		const float HitchThresholdSecs = GHitchThresholdMS * 0.001f;

		if ((GameThreadTime > HitchThresholdSecs) || (RenderThreadTime > HitchThresholdSecs))
		{
			TRACE_BOOKMARK(TEXT("Hitch detected"));
			FString OutputPathRel = FPaths::Combine(FPaths::ProfilingDir(), TEXT("Hitches"), FApp::GetInstanceId().ToString());
			FString OutputPath = IFileManager::Get().ConvertToAbsolutePathForExternalAppForWrite(*OutputPathRel);
			FString SnapshotFile = FPaths::Combine(OutputPath, FString::Printf(TEXT("Frame-%lld.utrace"), Frame));
			FString ScreenshotFile = FPaths::Combine(OutputPath, FString::Printf(TEXT("Frame-%lld.png"), Frame));
			//Write snapshot to target directory. This will also create directory in case it doesn't exist
			FTraceAuxiliary::WriteSnapshot(*SnapshotFile);
			FScreenshotRequest::RequestScreenshot(ScreenshotFile, true, false, false);
			UE_LOGF(LogEngine, Display, "Hitch detected on frame %lld, snapshot saved in '%ls'.", Frame, *SnapshotFile);

			LastHitchFrame = Frame;
		}
	}

private:
	static constexpr int32 ScreenshotFrameLag = 10;
	FDelegateHandle SnapshotHitchesHandle;
	
} GTraceHitechesCmd;

#endif //UE_SNAPSHOTHITCHES_ENABLED && STATS
