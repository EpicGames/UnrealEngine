// Copyright Epic Games, Inc. All Rights Reserved.
#include "ProfilingDebugging/TraceScreenshot.h"

#if UE_SCREENSHOT_TRACE_ENABLED

#include "ImageUtils.h"
#include "Misc/DateTime.h"
#include "Misc/CString.h"
#include "Misc/Paths.h"
#include "ProfilingDebugging/MiscTrace.h"
#include "Tasks/Task.h"
#include "UnrealClient.h"

#if WITH_EDITOR
#include "Editor.h"
#endif

static void TraceScreenshotCommandCallback(const TArray<FString>& Args)
{
	FString Name;
	bool bShowUI = false;

	if (Args.Num() == 1)
	{
		const FCoreTexts& CoreTexts = FCoreTexts::Get();

		// Try to see if the parameter is a name or a bool value for bShowUI.
		if (FCString::Stricmp(*Args[0], TEXT("true")) == 0 ||
		    FCString::Stricmp(*Args[0], TEXT("on")) == 0 ||
		    FCString::Stricmp(*Args[0], TEXT("yes")) == 0 ||
			FCString::Stricmp(*Args[0], TEXT("1")) == 0 ||
			FCString::Stricmp(*Args[0], *CoreTexts.True.ToString()) == 0 ||
			FCString::Stricmp(*Args[0], *CoreTexts.Yes.ToString()) == 0)
		{
			bShowUI = true;
		}
		else if (FCString::Stricmp(*Args[0], TEXT("false")) == 0 ||
				 FCString::Stricmp(*Args[0], TEXT("off")) == 0 ||
				 FCString::Stricmp(*Args[0], TEXT("no")) == 0 ||
				 FCString::Stricmp(*Args[0], TEXT("0")) == 0 ||
				 FCString::Stricmp(*Args[0], *CoreTexts.False.ToString()) == 0 ||
				 FCString::Stricmp(*Args[0], *CoreTexts.No.ToString()) == 0)
		{
			bShowUI = false;
		}
		else
		{
			Name = Args[0];
		}
	}
	else if (Args.Num() == 2)
	{
		Name = Args[0];
		bShowUI = FCString::ToBool(*Args[1]);
	}
	else if (Args.Num() > 2)
	{
		UE_LOGF(LogConsoleResponse, Warning, "Invalid arguments. Usage: Trace.Screenshot [Name] [bShowUI]");
		return;
	}

	FTraceScreenshot::RequestScreenshot(Name, bShowUI, LogConsoleResponse);
}

static FAutoConsoleCommand TraceScreenshotCmd(
	TEXT("Trace.Screenshot"),
	TEXT("[Name] [ShowUI] Takes a screenshot and saves it in the trace. Ex: Trace.Screenshot ScreenshotName true"),
	FConsoleCommandWithArgsDelegate::CreateStatic(TraceScreenshotCommandCallback)
);

bool FTraceScreenshot::bSuppressWritingToFile = false;

void FTraceScreenshot::RequestScreenshot(FString Name, bool bShowUI, const FLogCategoryAlias& LogCategory)
{
	if (!SHOULD_TRACE_SCREENSHOT())
	{
		UE_LOG_REF(LogCategory, Error, TEXT("Could not trace screenshot because the screenshot trace channel is off. Turn it on using \"Trace.Enable Screenshot\"."));
		return;
	}

	bSuppressWritingToFile = true;

	if (Name.IsEmpty())
	{
		Name = FDateTime::Now().ToString(TEXT("Screenshot_%Y%m%d_%H%M%S"));
	}
	constexpr bool bAddUniqueSuffix = false;
	FScreenshotRequest::RequestScreenshot(Name, bShowUI, bAddUniqueSuffix);

#if WITH_EDITOR
	// If not running PIE or Simulation we have to force a redraw.
	if (GEditor && !GEditor->IsPlayingSessionInEditor())
	{
		GEditor->RedrawAllViewports(/*bInvalidateHitProxies =*/false);
	}
#endif
}

void ImageUtilsImageResize(int32 SrcWidth, int32 SrcHeight, const TArray<FColor>& SrcData, int32 DstWidth, int32 DstHeight, TArray<FColor>& DstData, bool bLinearSpace)
{
	const bool bOpaqueAlpha = true;
	FImageUtils::ImageResize(SrcWidth, SrcHeight, SrcData, DstWidth, DstHeight, DstData, bLinearSpace, bOpaqueAlpha);
}

void ImageUtilsCompressImageArrayWrapper(int32 ImageWidth, int32 ImageHeight, const TArrayView64<const FColor>& SrcData, TArray64<uint8>& DstData)
{
	FImageUtils::PNGCompressImageArray(ImageWidth, ImageHeight, SrcData, DstData);
}

void ImageUtilsImageResize(int32 SrcWidth, int32 SrcHeight, const TArray64<FLinearColor>& SrcData, int32 DstWidth, int32 DstHeight, TArray64<FLinearColor>& DstData, bool /*bUnused*/)
{
	// note: does NOT opaque alpha
	FImageUtils::ImageResize(SrcWidth, SrcHeight, SrcData, DstWidth, DstHeight, DstData);
}

void ImageUtilsCompressImageArrayWrapper(int32 ImageWidth, int32 ImageHeight, const TArrayView64<const FLinearColor>& SrcData, TArray64<uint8>& DstData)
{
	FImageView TmpImageView(SrcData.GetData(), ImageWidth, ImageHeight);
	FImageUtils::CompressImage(DstData, TEXT(".exr"), TmpImageView);
}


template <class FColorType, class FResizedColorArrayType, typename TChannelType>
void TraceScreenshotInternal(int32 InSizeX, int32 InSizeY, const TArray<FColorType>& InImageData, const FString& InScreenshotName, int32 DesiredX, 
							 TChannelType OpaqueAlphaValue)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(ScreenshotTracing_Prepare);
	FString ScreenshotName = FPaths::GetBaseFilename(InScreenshotName);

	if (ScreenshotName.IsEmpty())
	{
		ScreenshotName = FDateTime::Now().ToString(TEXT("Screenshot_%Y%m%d_%H%M%S"));
	}

	UE_LOGF(LogCore, Display, "Tracing Screenshot \"%ls\" taken with size: %d x %d", *ScreenshotName, InSizeX, InSizeY);

	uint64 Cycles = FPlatformTime::Cycles64();
	FResizedColorArrayType* ImageCopy = new FResizedColorArrayType(InImageData);

	UE::Tasks::Launch(UE_SOURCE_LOCATION,
		[ImageCopy, Cycles, ScreenshotName, InSizeX, InSizeY, DesiredX, OpaqueAlphaValue]
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(ScreenshotTracing_Execute);

			// Set full alpha on the bitmap
			for (FColorType& Pixel : *ImageCopy)
			{
				Pixel.A = OpaqueAlphaValue;
			}

			TArray64<uint8> CompressedBitmap;
			if (DesiredX > 0 && InSizeX != DesiredX)
			{
				int32 ResizedX = FMath::Min(640, InSizeX);
				int32 ResizedY = (InSizeY * ResizedX) / InSizeX;

				FResizedColorArrayType ResizedImage;
				ResizedImage.SetNum(ResizedX * ResizedY);
				ImageUtilsImageResize(InSizeX, InSizeY, *ImageCopy, ResizedX, ResizedY, ResizedImage, true);

				ImageUtilsCompressImageArrayWrapper(ResizedX, ResizedY, TArrayView64<const FColorType>(ResizedImage.GetData(), ResizedImage.Num()), CompressedBitmap);
				TRACE_SCREENSHOT(*ScreenshotName, Cycles, ResizedX, ResizedY, CompressedBitmap);
			}
			else
			{
				ImageUtilsCompressImageArrayWrapper(InSizeX, InSizeY, TArrayView64<const FColorType>(ImageCopy->GetData(), ImageCopy->Num()), CompressedBitmap);
				TRACE_SCREENSHOT(*ScreenshotName, Cycles, InSizeX, InSizeY, CompressedBitmap);
			}

			delete ImageCopy;
		});

}

void FTraceScreenshot::TraceScreenshot(int32 InSizeX, int32 InSizeY, const TArray<FColor>& InImageData, const FString& InScreenshotName, int32 DesiredX)
{
	TraceScreenshotInternal<FColor, TArray<FColor>, uint8>(InSizeX, InSizeY, InImageData, InScreenshotName, DesiredX, 255);
	Reset();
}

void FTraceScreenshot::TraceScreenshot(int32 InSizeX, int32 InSizeY, const TArray<FLinearColor>& InImageData, const FString& InScreenshotName, int32 DesiredX)
{
	TraceScreenshotInternal<FLinearColor, TArray64<FLinearColor>, float>(InSizeX, InSizeY, InImageData, InScreenshotName, DesiredX, 1.0f);
	Reset();
}

void FTraceScreenshot::Reset()
{
	bSuppressWritingToFile = false;
}

#endif // UE_SCREENSHOT_TRACE_ENABLED
