// Copyright Epic Games, Inc. All Rights Reserved.

#include "ProfilingDebugging/WorldStreamingPackageDependencyUtil.h"

#if UE_TRACE_WORLD_STREAMING_DEPENDENCIES_ENABLED

#include "Async/Async.h"
#include "IPlatformFilePak.h"
#include "Misc/PackageName.h"
#include "ProfilingDebugging/CpuProfilerTrace.h"
#include "Serialization/PackageStore.h"
#include "Tasks/Task.h"

FWorldStreamingPackageIdNameMap::~FWorldStreamingPackageIdNameMap()
{
	BuildTask.Wait();
}

FWorldStreamingPackageIdNameMap& FWorldStreamingPackageIdNameMap::Get()
{
	check(IsInGameThread());
	static TSharedRef<FWorldStreamingPackageIdNameMap> Instance = []()
	{
		TSharedRef<FWorldStreamingPackageIdNameMap> NewInstance = MakeShared<FWorldStreamingPackageIdNameMap>();
		NewInstance->RegisterDelegates();
		return NewInstance;
	}();
	return Instance.Get();
}

void FWorldStreamingPackageIdNameMap::RegisterDelegates()
{
	// Dismount is intentionally not subscribed: FPackageId::FromName is deterministic, so entries for unmounted packages remain factually correct name mappings.
	FPackageName::OnContentPathMounted().AddSP(this, &FWorldStreamingPackageIdNameMap::OnContentPathMounted);
}

void FWorldStreamingPackageIdNameMap::BuildDatabaseAsync()
{
	check(IsInGameThread());

	if (bBuildTaskInFlight || IsReady())
	{
		return;
	}

	bBuildTaskInFlight = true;
	TWeakPtr<FWorldStreamingPackageIdNameMap> WeakThis = AsShared();
	BuildTask = UE::Tasks::Launch(
		UE_SOURCE_LOCATION,
		[WeakThis]()
		{
			if (TSharedPtr<FWorldStreamingPackageIdNameMap> PinnedThis = WeakThis.Pin())
			{
				PinnedThis->BuildDatabase();
			}

			AsyncTask(ENamedThreads::GameThread, [WeakThis]()
			{
				if (TSharedPtr<FWorldStreamingPackageIdNameMap> PinnedThis = WeakThis.Pin())
				{
					PinnedThis->bBuildTaskInFlight = false;
					if (PinnedThis->IsReady())
					{
						PinnedThis->OnDatabaseReady.Broadcast();
					}
					else
					{
						PinnedThis->BuildDatabaseAsync();
					}
				}
			});
		});
}

bool FWorldStreamingPackageIdNameMap::IsReady() const
{
	// Pairs with the seq_cst store in BuildDatabase: observing equality means the Database writes are visible to GT readers.
	return DatabaseGeneration.load(std::memory_order_seq_cst) == MountedContentGeneration.load(std::memory_order_seq_cst);
}

bool FWorldStreamingPackageIdNameMap::GetPackageNameFromId(FPackageId InPackageId, FName& OutFullName) const
{
	check(IsInGameThread());
	if (!ensure(IsReady()))
	{
		return false;
	}

	if (const FName* Found = Database.Find(InPackageId))
	{
		OutFullName = *Found;
		return true;
	}

	return false;
}

void FWorldStreamingPackageIdNameMap::BuildDatabase()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FWorldStreamingPackageIdNameMap::BuildDatabase);

	check(!IsReady());

	const uint32 StartGeneration = MountedContentGeneration.load(std::memory_order_seq_cst);

	FPakPlatformFile::ForeachPackageInIostoreWhile(
		[this](FName PackageName)
		{
			Database.FindOrAdd(FPackageId::FromName(PackageName), PackageName);
			return true;
		});

	// Commit the generation the scan started at. The seq_cst store publishes the Database writes above to GT readers that observe IsReady() == true.
	// If a mount fired during or after the scan, MountedContentGeneration has already advanced past StartGeneration
	// and IsReady() will return false — the retry path (BuildDatabaseAsync continuation) re-reads the current generation and scans again.
	DatabaseGeneration.store(StartGeneration, std::memory_order_seq_cst);
}

void FWorldStreamingPackageIdNameMap::InvalidateDatabase()
{
	// Monotonic bump. Readers observe DatabaseGeneration < MountedContentGeneration until the next successful scan catches up.
	// Database entries stay — FPackageId::FromName is deterministic, so cached entries never go stale.
	MountedContentGeneration.fetch_add(1, std::memory_order_seq_cst);
}

void FWorldStreamingPackageIdNameMap::OnContentPathMounted(const FString& InContentPath, const FString& InRootPath)
{
	InvalidateDatabase();
}

FWorldStreamingPackageDependencyCache& FWorldStreamingPackageDependencyCache::Get()
{
	check(IsInGameThread());
	static TSharedRef<FWorldStreamingPackageDependencyCache> Instance = []()
	{
		TSharedRef<FWorldStreamingPackageDependencyCache> NewInstance = MakeShared<FWorldStreamingPackageDependencyCache>();
		NewInstance->RegisterDelegates();
		return NewInstance;
	}();
	return Instance.Get();
}

void FWorldStreamingPackageDependencyCache::RegisterDelegates()
{
	FPackageName::OnContentPathMounted().AddSP(this, &FWorldStreamingPackageDependencyCache::OnContentPathMounted);
	FPackageName::OnContentPathDismounted().AddSP(this, &FWorldStreamingPackageDependencyCache::OnContentPathDismounted);
}

bool FWorldStreamingPackageDependencyCache::GetDependencies(FPackageId InPackageId, TSet<FPackageId>& OutDependencies)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FWorldStreamingPackageDependencyCache::GetDependencies);

	check(IsInGameThread());

	OutDependencies.Reset();

	const TSet<FPackageId>* DirectDependencies = Dependencies.Find(InPackageId);
	if (DirectDependencies == nullptr)
	{
		FPackageStoreReadScope ReadScope(FPackageStore::Get());
		if (!InsertPackage(InPackageId))
		{
			return false;
		}
		DirectDependencies = Dependencies.Find(InPackageId);
	}

	TArray<FPackageId> PackagesToProcess;
	for (const FPackageId& DependencyId : *DirectDependencies)
	{
		PackagesToProcess.Add(DependencyId);
	}

	while (!PackagesToProcess.IsEmpty())
	{
		FPackageId DependencyId = PackagesToProcess.Pop(EAllowShrinking::No);
		if (!OutDependencies.Contains(DependencyId))
		{
			OutDependencies.Add(DependencyId);
			if (const TSet<FPackageId>* TransitiveDependencies = Dependencies.Find(DependencyId))
			{
				for (const FPackageId& TransitiveId : *TransitiveDependencies)
				{
					PackagesToProcess.Add(TransitiveId);
				}
			}
		}
	}

	return true;
}

void FWorldStreamingPackageDependencyCache::ResetDatabase()
{
	check(IsInGameThread());
	Dependencies.Empty();
}

void FWorldStreamingPackageDependencyCache::OnContentPathMounted(const FString& InContentPath, const FString& InRootPath)
{
	// Delegate can fire from any thread; ResetDatabase mutates Dependencies which is read on GT.
	TWeakPtr<FWorldStreamingPackageDependencyCache> WeakThis = AsShared();
	AsyncTask(ENamedThreads::GameThread, [WeakThis]()
	{
		if (TSharedPtr<FWorldStreamingPackageDependencyCache> PinnedThis = WeakThis.Pin())
		{
			PinnedThis->ResetDatabase();
		}
	});
}

void FWorldStreamingPackageDependencyCache::OnContentPathDismounted(const FString& InContentPath, const FString& InRootPath)
{
	TWeakPtr<FWorldStreamingPackageDependencyCache> WeakThis = AsShared();
	AsyncTask(ENamedThreads::GameThread, [WeakThis]()
	{
		if (TSharedPtr<FWorldStreamingPackageDependencyCache> PinnedThis = WeakThis.Pin())
		{
			PinnedThis->ResetDatabase();
		}
	});
}

bool FWorldStreamingPackageDependencyCache::InsertPackage(FPackageId InRootPackageId)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FWorldStreamingPackageDependencyCache::InsertPackage);

	TArray<FPackageId> PackagesToProcess;
	PackagesToProcess.Add(InRootPackageId);
	bool bSuccess = false;

	while (!PackagesToProcess.IsEmpty())
	{
		const FPackageId PackageId = PackagesToProcess.Pop(EAllowShrinking::No);

		if (Dependencies.Contains(PackageId))
		{
			bSuccess = true;
			continue;
		}

		FPackageStoreEntry PackageEntry;
		const EPackageStoreEntryStatus Status = FPackageStore::Get().GetPackageStoreEntry(PackageId, NAME_None, PackageEntry);
		if (Status == EPackageStoreEntryStatus::Ok)
		{
			const TArrayView<const FPackageId> ImportedPackageIds = PackageEntry.ImportedPackageIds;
			TSet<FPackageId>& PackageDependencies = Dependencies.FindOrAdd(PackageId);
			for (FPackageId ImportedPackageId : ImportedPackageIds)
			{
				PackageDependencies.Add(ImportedPackageId);
			}
			PackagesToProcess.Append(ImportedPackageIds);
			bSuccess = true;
		}
	}

	return bSuccess;
}

#endif // UE_TRACE_WORLD_STREAMING_DEPENDENCIES_ENABLED
