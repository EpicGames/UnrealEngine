// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ProfilingDebugging/WorldStreamingTrace.h"

#if UE_TRACE_WORLD_STREAMING_DEPENDENCIES_ENABLED

#include "IO/PackageId.h"
#include "Tasks/Task.h"

// PackageId to FName map and PackageStore dependency cache for World Streaming trace emission.

DECLARE_MULTICAST_DELEGATE(FOnWorldStreamingPackageIdNameMapReady);

// Maps FPackageId → FName by enumerating all packages from IoStore. Call BuildDatabaseAsync to start the scan on a worker thread.
// Subscribe to OnDatabaseReady to be notified when the scan completes. GetPackageNameFromId is a pure lookup — only valid after IsReady() returns true.
//
// Database is append-only at runtime: FPackageId::FromName is a deterministic CityHash64 of the FName, so cached (id → name) pairs never go stale.
// Mounts add entries; dismounts don't invalidate. InvalidateDatabase only bumps the generation counter; the next scan merges newly-mounted paks' entries additively via FindOrAdd.
class FWorldStreamingPackageIdNameMap : public TSharedFromThis<FWorldStreamingPackageIdNameMap>
{
public:
	static FWorldStreamingPackageIdNameMap& Get();

	void BuildDatabaseAsync();
	bool IsReady() const;
	bool GetPackageNameFromId(FPackageId InPackageId, FName& OutFullName) const;

	FOnWorldStreamingPackageIdNameMapReady OnDatabaseReady;

	~FWorldStreamingPackageIdNameMap();

private:
	void RegisterDelegates();
	void BuildDatabase();
	void InvalidateDatabase();
	void OnContentPathMounted(const FString& InContentPath, const FString& InRootPath);

	// Incremented on every content-path mount event (FPackageName::OnContentPathMounted).
	// Tracks the version of the currently-mounted content — bumps whenever a new package root is mounted. Initial value 1 so Database starts not-ready.
	std::atomic<uint32> MountedContentGeneration{1};

	// The MountedContentGeneration that the Database was built against.
	// Database is valid (IsReady() == true) when DatabaseGeneration == MountedContentGeneration.
	std::atomic<uint32> DatabaseGeneration{0};

	TMap<FPackageId, FName> Database;

	UE::Tasks::FTask BuildTask;
	bool bBuildTaskInFlight = false;
};

// Caches transitive package dependencies from PackageStore.
// GetDependencies returns the full transitive closure for a package, populating the cache via DFS on first miss.
// Game-thread-only — no internal locking.
class FWorldStreamingPackageDependencyCache : public TSharedFromThis<FWorldStreamingPackageDependencyCache>
{
public:
	static FWorldStreamingPackageDependencyCache& Get();

	bool GetDependencies(FPackageId InPackageId, TSet<FPackageId>& OutDependencies);

private:
	void RegisterDelegates();
	void ResetDatabase();
	void OnContentPathMounted(const FString& InContentPath, const FString& InRootPath);
	void OnContentPathDismounted(const FString& InContentPath, const FString& InRootPath);
	bool InsertPackage(FPackageId InRootPackageId);

	TMap<FPackageId, TSet<FPackageId>> Dependencies;
};

#endif // UE_TRACE_WORLD_STREAMING_DEPENDENCIES_ENABLED
