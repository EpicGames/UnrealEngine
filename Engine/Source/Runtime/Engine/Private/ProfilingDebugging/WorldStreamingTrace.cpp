// Copyright Epic Games, Inc. All Rights Reserved.

#include "ProfilingDebugging/WorldStreamingTrace.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(WorldStreamingTrace)

#if UE_TRACE_WORLD_STREAMING_ENABLED

DEFINE_LOG_CATEGORY(LogWorldStreamingTrace);

#include "Engine/World.h"
#include "HAL/IConsoleManager.h"
#include "Hash/xxhash.h"
#include "Misc/PackageName.h"
#include "ProfilingDebugging/CpuProfilerTrace.h"
#include "Trace/Trace.inl"

#if UE_TRACE_WORLD_STREAMING_DEPENDENCIES_ENABLED
#include "ProfilingDebugging/WorldStreamingPackageDependencyUtil.h"
#include "Serialization/PackageStore.h"
#endif // UE_TRACE_WORLD_STREAMING_DEPENDENCIES_ENABLED

static float GWorldStreamingTraceSourceLocationChangeThreshold = 100.f;
static FAutoConsoleVariableRef CVarWorldStreamingTraceSourceLocationChangeThreshold(
	TEXT("Trace.WorldStreaming.SourceLocationChangeThreshold"),
	GWorldStreamingTraceSourceLocationChangeThreshold,
	TEXT("Minimum distance change to trace streaming source update"));

#if UE_TRACE_WORLD_STREAMING_PRIORITY_ENABLED
static float GWorldStreamingTracePriorityUpdateInterval = 1.f;
static FAutoConsoleVariableRef CVarWorldStreamingTracePriorityUpdateInterval(
	TEXT("Trace.WorldStreaming.PriorityUpdateInterval"),
	GWorldStreamingTracePriorityUpdateInterval,
	TEXT("Minimum interval in seconds between priority trace emissions"));
#endif // UE_TRACE_WORLD_STREAMING_PRIORITY_ENABLED

static uint64 WorldInstanceCounter = 0;

UE_TRACE_MINIMAL_CHANNEL_DEFINE(WorldStreamingChannel)

#if UE_TRACE_WORLD_STREAMING_PRIORITY_ENABLED
UE_TRACE_MINIMAL_CHANNEL_DEFINE(WorldStreamingPriorityChannel)
#endif // UE_TRACE_WORLD_STREAMING_PRIORITY_ENABLED

#if UE_TRACE_WORLD_STREAMING_DEPENDENCIES_ENABLED
UE_TRACE_MINIMAL_CHANNEL_DEFINE(WorldStreamingDependenciesChannel)
#endif // UE_TRACE_WORLD_STREAMING_DEPENDENCIES_ENABLED

// EXPERIMENTAL: trace event names, field names, and field types declared below are subject to change without deprecation.
// Data captured now may not display correctly in future engine versions.

UE_TRACE_MINIMAL_EVENT_BEGIN(WorldStreaming, WorldInitialization)
	UE_TRACE_MINIMAL_EVENT_FIELD(uint64, WorldId)
	UE_TRACE_MINIMAL_EVENT_FIELD(uint64, Cycle)
	UE_TRACE_MINIMAL_EVENT_FIELD(UE::Trace::WideString, MapName)
	UE_TRACE_MINIMAL_EVENT_FIELD(uint8, NetMode)
UE_TRACE_MINIMAL_EVENT_END()

UE_TRACE_MINIMAL_EVENT_BEGIN(WorldStreaming, WorldDeinitialization)
	UE_TRACE_MINIMAL_EVENT_FIELD(uint64, WorldId)
	UE_TRACE_MINIMAL_EVENT_FIELD(uint64, Cycle)
UE_TRACE_MINIMAL_EVENT_END()

UE_TRACE_MINIMAL_EVENT_BEGIN(WorldStreaming, ContainerDescription)
	UE_TRACE_MINIMAL_EVENT_FIELD(uint64, Id)
	UE_TRACE_MINIMAL_EVENT_FIELD(uint64, WorldId)
	UE_TRACE_MINIMAL_EVENT_FIELD(uint64, ParentId)
	UE_TRACE_MINIMAL_EVENT_FIELD(UE::Trace::WideString, Name)
	UE_TRACE_MINIMAL_EVENT_FIELD(UE::Trace::WideString, PackageName)
	UE_TRACE_MINIMAL_EVENT_FIELD(uint8, bBoundsValid)
	UE_TRACE_MINIMAL_EVENT_FIELD(double[], Bounds)
	UE_TRACE_MINIMAL_EVENT_FIELD(uint64[], Tags)
UE_TRACE_MINIMAL_EVENT_END()

UE_TRACE_MINIMAL_EVENT_BEGIN(WorldStreaming, ContainerStateChange)
	UE_TRACE_MINIMAL_EVENT_FIELD(uint64, Id)
	UE_TRACE_MINIMAL_EVENT_FIELD(uint64, Cycle)
	UE_TRACE_MINIMAL_EVENT_FIELD(uint64, WorldId)
	UE_TRACE_MINIMAL_EVENT_FIELD(uint8, NewState)
UE_TRACE_MINIMAL_EVENT_END()

UE_TRACE_MINIMAL_EVENT_BEGIN(WorldStreaming, StreamingSourceDescription)
	UE_TRACE_MINIMAL_EVENT_FIELD(uint64, Id)
	UE_TRACE_MINIMAL_EVENT_FIELD(uint64, WorldId)
	UE_TRACE_MINIMAL_EVENT_FIELD(UE::Trace::WideString, Name)
UE_TRACE_MINIMAL_EVENT_END()

UE_TRACE_MINIMAL_EVENT_BEGIN(WorldStreaming, StreamingSourceUpdate)
	UE_TRACE_MINIMAL_EVENT_FIELD(uint64, Id)
	UE_TRACE_MINIMAL_EVENT_FIELD(uint64, Cycle)
	UE_TRACE_MINIMAL_EVENT_FIELD(uint64, WorldId)
	UE_TRACE_MINIMAL_EVENT_FIELD(double[], Location)
UE_TRACE_MINIMAL_EVENT_END()

UE_TRACE_MINIMAL_EVENT_BEGIN(WorldStreaming, StreamingSourceDeactivation)
	UE_TRACE_MINIMAL_EVENT_FIELD(uint64, Id)
	UE_TRACE_MINIMAL_EVENT_FIELD(uint64, Cycle)
	UE_TRACE_MINIMAL_EVENT_FIELD(uint64, WorldId)
UE_TRACE_MINIMAL_EVENT_END()

UE_TRACE_MINIMAL_EVENT_BEGIN(WorldStreaming, TagGroupDescription)
	UE_TRACE_MINIMAL_EVENT_FIELD(uint64, Id)
	UE_TRACE_MINIMAL_EVENT_FIELD(UE::Trace::WideString, Name)
UE_TRACE_MINIMAL_EVENT_END()

UE_TRACE_MINIMAL_EVENT_BEGIN(WorldStreaming, TagDescription)
	UE_TRACE_MINIMAL_EVENT_FIELD(uint64, Id)
	UE_TRACE_MINIMAL_EVENT_FIELD(uint64, GroupId)
	UE_TRACE_MINIMAL_EVENT_FIELD(uint64, ParentId)
	UE_TRACE_MINIMAL_EVENT_FIELD(UE::Trace::WideString, Name)
UE_TRACE_MINIMAL_EVENT_END()

#if UE_TRACE_WORLD_STREAMING_PRIORITY_ENABLED
UE_TRACE_MINIMAL_EVENT_BEGIN(WorldStreaming, ContainerPriorityUpdate)
	UE_TRACE_MINIMAL_EVENT_FIELD(uint64, WorldId)
	UE_TRACE_MINIMAL_EVENT_FIELD(uint64, Cycle)
	UE_TRACE_MINIMAL_EVENT_FIELD(uint64[], ContainerIds)
	UE_TRACE_MINIMAL_EVENT_FIELD(float[], Priorities)
UE_TRACE_MINIMAL_EVENT_END()
#endif // UE_TRACE_WORLD_STREAMING_PRIORITY_ENABLED

#if UE_TRACE_WORLD_STREAMING_DEPENDENCIES_ENABLED
UE_TRACE_MINIMAL_EVENT_BEGIN(WorldStreaming, PackageNameMapping)
	UE_TRACE_MINIMAL_EVENT_FIELD(uint64, PackageId)
	UE_TRACE_MINIMAL_EVENT_FIELD(UE::Trace::WideString, Name)
UE_TRACE_MINIMAL_EVENT_END()

UE_TRACE_MINIMAL_EVENT_BEGIN(WorldStreaming, ContainerDependencies)
	UE_TRACE_MINIMAL_EVENT_FIELD(uint64, Id)
	UE_TRACE_MINIMAL_EVENT_FIELD(uint64, WorldId)
	UE_TRACE_MINIMAL_EVENT_FIELD(uint64[], DependencyIds)
UE_TRACE_MINIMAL_EVENT_END()
#endif // UE_TRACE_WORLD_STREAMING_DEPENDENCIES_ENABLED
#endif // UE_TRACE_WORLD_STREAMING_ENABLED

bool UWorldStreamingTraceSubsystem::ShouldCreateSubsystem(UObject* Outer) const
{
#if UE_TRACE_WORLD_STREAMING_ENABLED
	if (!UE_TRACE_MINIMAL_CHANNELEXPR_IS_ENABLED(WorldStreamingChannel))
	{
		return false;
	}

	const UWorld* World = Cast<UWorld>(Outer);
	if (!World)
	{
		return false;
	}

	if (FPackageName::IsTempPackage(World->GetPackage()->GetName()))
	{
		return false;
	}

	if (IsRunningDedicatedServer() || World->IsNetMode(NM_DedicatedServer))
	{
		return false;
	}

	return Super::ShouldCreateSubsystem(Outer);
#else
	return false;
#endif
}

bool UWorldStreamingTraceSubsystem::DoesSupportWorldType(const EWorldType::Type WorldType) const
{
	return WorldType == EWorldType::Game || WorldType == EWorldType::PIE;
}

#if UE_TRACE_WORLD_STREAMING_ENABLED
void UWorldStreamingTraceSubsystem::Initialize(FSubsystemCollectionBase& Collection)
{
	Super::Initialize(Collection);

	UE_LOGF(LogWorldStreamingTrace, Log, "WorldStreaming trace is experimental; data captured now may not display correctly in future engine versions.");

	const UWorld* World = GetWorld();
	const uint64 Cycle = FPlatformTime::Cycles64();
	const FString MapName = World->GetMapName();
	const uint64 InstanceId = WorldInstanceCounter++;
	const uint8 NetMode = static_cast<uint8>(World->GetNetMode());
	const FString WorldPackageName = World->GetPackage()->GetName();

	FXxHash64Builder Builder;
	Builder.Update(*MapName, MapName.Len() * sizeof(TCHAR));
	Builder.Update(&InstanceId, sizeof(InstanceId));
	WorldId = Builder.Finalize().Hash;

	UE_TRACE_MINIMAL_LOG(WorldStreaming, WorldInitialization, WorldStreamingChannel)
		<< WorldInitialization.WorldId(WorldId)
		<< WorldInitialization.Cycle(Cycle)
		<< WorldInitialization.MapName(*MapName, MapName.Len())
		<< WorldInitialization.NetMode(NetMode);

	const TArray<uint64> EmptyTags;
	EnsureContainerDescribed(WorldId, /*InParentId=*/0, *MapName, *WorldPackageName, FBox(ForceInit), EmptyTags);
	TraceContainerStateChange(WorldId, UE::WorldStreamingTrace::EStreamingContainerState::Active);

#if UE_TRACE_WORLD_STREAMING_DEPENDENCIES_ENABLED
	if (UE_TRACE_MINIMAL_CHANNELEXPR_IS_ENABLED(WorldStreamingDependenciesChannel))
	{
		FWorldStreamingPackageIdNameMap::Get().OnDatabaseReady.AddUObject(this, &UWorldStreamingTraceSubsystem::OnNameMappingBuildComplete);
		TraceDependenciesForPackage(WorldId, World->GetPackage()->GetFName());
	}
#endif // UE_TRACE_WORLD_STREAMING_DEPENDENCIES_ENABLED
}

void UWorldStreamingTraceSubsystem::Deinitialize()
{
#if UE_TRACE_WORLD_STREAMING_DEPENDENCIES_ENABLED
	FWorldStreamingPackageIdNameMap::Get().OnDatabaseReady.RemoveAll(this);
#endif

	const uint64 Cycle = FPlatformTime::Cycles64();

	TraceContainerStateChange(WorldId, UE::WorldStreamingTrace::EStreamingContainerState::Unloaded);

	UE_TRACE_MINIMAL_LOG(WorldStreaming, WorldDeinitialization, WorldStreamingChannel)
		<< WorldDeinitialization.WorldId(WorldId)
		<< WorldDeinitialization.Cycle(Cycle);

	Super::Deinitialize();
}

void UWorldStreamingTraceSubsystem::TraceContainerStateChange(uint64 InContainerId, UE::WorldStreamingTrace::EStreamingContainerState InState)
{
	check(IsInGameThread());
	const uint64 Cycle = FPlatformTime::Cycles64();

	UE_TRACE_MINIMAL_LOG(WorldStreaming, ContainerStateChange, WorldStreamingChannel)
		<< ContainerStateChange.Id(InContainerId)
		<< ContainerStateChange.Cycle(Cycle)
		<< ContainerStateChange.WorldId(WorldId)
		<< ContainerStateChange.NewState(static_cast<uint8>(InState));
}

void UWorldStreamingTraceSubsystem::EnsureContainerDescribed(uint64 InContainerId, uint64 InParentId, FStringView InName, FStringView InPackageName, const FBox& InBounds, const TArray<uint64>& InTags)
{
	check(IsInGameThread());
	if (!KnownContainers.Contains(InContainerId))
	{
		KnownContainers.Add(InContainerId);
		TraceContainerDescription(InContainerId, InParentId, InName, InPackageName, InBounds, InTags);
	}
}

void UWorldStreamingTraceSubsystem::TraceContainerDescription(uint64 InContainerId, uint64 InParentId, FStringView InName, FStringView InPackageName, const FBox& InBounds, const TArray<uint64>& InTags)
{
	const double BoundsData[6] = {
		InBounds.Min.X, InBounds.Min.Y, InBounds.Min.Z,
		InBounds.Max.X, InBounds.Max.Y, InBounds.Max.Z
	};

	UE_TRACE_MINIMAL_LOG(WorldStreaming, ContainerDescription, WorldStreamingChannel)
		<< ContainerDescription.Id(InContainerId)
		<< ContainerDescription.WorldId(WorldId)
		<< ContainerDescription.ParentId(InParentId)
		<< ContainerDescription.Name(InName.GetData(), InName.Len())
		<< ContainerDescription.PackageName(InPackageName.GetData(), InPackageName.Len())
		<< ContainerDescription.bBoundsValid(InBounds.IsValid ? 1 : 0)
		<< ContainerDescription.Bounds(BoundsData, 6)
		<< ContainerDescription.Tags(InTags.GetData(), InTags.Num());
}

bool UWorldStreamingTraceSubsystem::IsContainerDescribed(uint64 InContainerId) const
{
	check(IsInGameThread());
	return KnownContainers.Contains(InContainerId);
}

bool UWorldStreamingTraceSubsystem::IsStreamingSourceDescribed(uint64 InStreamingSourceId) const
{
	check(IsInGameThread());
	return KnownStreamingSources.Contains(InStreamingSourceId);
}

void UWorldStreamingTraceSubsystem::TraceStreamingSourceUpdate(uint64 InStreamingSourceId, const FVector& InLocation)
{
	check(IsInGameThread());
	FVector LastLocation;
	bool bFirstUpdate = false;
	if (FVector* LastLocationPtr = StreamingSourceLocations.Find(InStreamingSourceId))
	{
		LastLocation = *LastLocationPtr;
	}
	else
	{
		bFirstUpdate = true;
		StreamingSourceLocations.Add(InStreamingSourceId, InLocation);
	}

	ActiveStreamingSources.Add(InStreamingSourceId);

	const bool bExceededLocationChangeThreshold = FVector::DistSquared(InLocation, LastLocation) >= FMath::Square(GWorldStreamingTraceSourceLocationChangeThreshold);
	if (bFirstUpdate || bExceededLocationChangeThreshold)
	{
		StreamingSourceLocations[InStreamingSourceId] = InLocation;

		const uint64 Cycle = FPlatformTime::Cycles64();
		const double LocationData[3] = { InLocation.X, InLocation.Y, InLocation.Z };

		UE_TRACE_MINIMAL_LOG(WorldStreaming, StreamingSourceUpdate, WorldStreamingChannel)
			<< StreamingSourceUpdate.Id(InStreamingSourceId)
			<< StreamingSourceUpdate.Cycle(Cycle)
			<< StreamingSourceUpdate.WorldId(WorldId)
			<< StreamingSourceUpdate.Location(LocationData, 3);
	}
}

void UWorldStreamingTraceSubsystem::EnsureStreamingSourceDescribed(uint64 InStreamingSourceId, FStringView InName)
{
	check(IsInGameThread());
	if (!KnownStreamingSources.Contains(InStreamingSourceId))
	{
		KnownStreamingSources.Add(InStreamingSourceId);
		TraceStreamingSourceDescription(InStreamingSourceId, InName);
	}
}

void UWorldStreamingTraceSubsystem::TraceStreamingSourceDescription(uint64 InStreamingSourceId, FStringView InName)
{
	UE_TRACE_MINIMAL_LOG(WorldStreaming, StreamingSourceDescription, WorldStreamingChannel)
		<< StreamingSourceDescription.Id(InStreamingSourceId)
		<< StreamingSourceDescription.WorldId(WorldId)
		<< StreamingSourceDescription.Name(InName.GetData(), InName.Len());
}

void UWorldStreamingTraceSubsystem::FlushInactiveStreamingSources()
{
	check(IsInGameThread());
	for (TMap<uint64, FVector>::TIterator It(StreamingSourceLocations); It; ++It)
	{
		if (!ActiveStreamingSources.Contains(It->Key))
		{
			const uint64 Cycle = FPlatformTime::Cycles64();
			
			UE_TRACE_MINIMAL_LOG(WorldStreaming, StreamingSourceDeactivation, WorldStreamingChannel)
				<< StreamingSourceDeactivation.Id(It->Key)
				<< StreamingSourceDeactivation.Cycle(Cycle)
				<< StreamingSourceDeactivation.WorldId(WorldId);

			It.RemoveCurrent();
		}
	}
	
	ActiveStreamingSources.Reset();
}

void UWorldStreamingTraceSubsystem::EnsureTagGroupDescribed(uint64 InTagGroupId, FStringView InTagGroupName)
{
	check(IsInGameThread());
	if (!KnownTagGroups.Contains(InTagGroupId))
	{
		KnownTagGroups.Add(InTagGroupId);
		TraceTagGroupDescription(InTagGroupId, InTagGroupName);
	}
}

void UWorldStreamingTraceSubsystem::EnsureTagDescribed(uint64 InTagId, uint64 InTagGroupId, uint64 InParentId, FStringView InName)
{
	check(IsInGameThread());
	if (!KnownTags.Contains(InTagId))
	{
		KnownTags.Add(InTagId);
		TraceTagDescription(InTagId, InTagGroupId, InParentId, InName);
	}
}

void UWorldStreamingTraceSubsystem::TraceTagGroupDescription(uint64 InTagGroupId, FStringView InTagGroupName)
{
	UE_TRACE_MINIMAL_LOG(WorldStreaming, TagGroupDescription, WorldStreamingChannel)
		<< TagGroupDescription.Id(InTagGroupId)
		<< TagGroupDescription.Name(InTagGroupName.GetData(), InTagGroupName.Len());
}

void UWorldStreamingTraceSubsystem::TraceTagDescription(uint64 InTagId, uint64 InTagGroupId, uint64 InParentId, FStringView InName)
{
	UE_TRACE_MINIMAL_LOG(WorldStreaming, TagDescription, WorldStreamingChannel)
		<< TagDescription.Id(InTagId)
		<< TagDescription.GroupId(InTagGroupId)
		<< TagDescription.ParentId(InParentId)
		<< TagDescription.Name(InName.GetData(), InName.Len());
}

#if UE_TRACE_WORLD_STREAMING_PRIORITY_ENABLED
bool UWorldStreamingTraceSubsystem::ShouldTraceContainerPriorityThisFrame(const UWorld* InWorld)
{
	check(IsInGameThread());
	if (!UE_TRACE_MINIMAL_CHANNELEXPR_IS_ENABLED(WorldStreamingPriorityChannel))
	{
		return false;
	}

	UWorldStreamingTraceSubsystem* Subsystem = UWorld::GetSubsystem<UWorldStreamingTraceSubsystem>(InWorld);
	if (!Subsystem)
	{
		return false;
	}

	const double Now = FPlatformTime::Seconds();
	return (Now - Subsystem->LastPriorityEmitTime) >= GWorldStreamingTracePriorityUpdateInterval;
}

void UWorldStreamingTraceSubsystem::TraceContainerPriorityBatch(TArrayView<const uint64> InContainerIds, TArrayView<const float> InPriorities)
{
	check(IsInGameThread());
	check(InContainerIds.Num() == InPriorities.Num());

	if (InContainerIds.Num() == 0)
	{
		return;
	}

	// Throttle is enforced via ShouldTraceContainerPriorityThisFrame before capture; this just records the emit time for the next decision.
	LastPriorityEmitTime = FPlatformTime::Seconds();

	const uint64 Cycle = FPlatformTime::Cycles64();

	UE_TRACE_MINIMAL_LOG(WorldStreaming, ContainerPriorityUpdate, WorldStreamingPriorityChannel)
		<< ContainerPriorityUpdate.WorldId(WorldId)
		<< ContainerPriorityUpdate.Cycle(Cycle)
		<< ContainerPriorityUpdate.ContainerIds(InContainerIds.GetData(), InContainerIds.Num())
		<< ContainerPriorityUpdate.Priorities(InPriorities.GetData(), InPriorities.Num());
}
#endif // UE_TRACE_WORLD_STREAMING_PRIORITY_ENABLED

#if UE_TRACE_WORLD_STREAMING_DEPENDENCIES_ENABLED
void UWorldStreamingTraceSubsystem::TraceDependenciesForPackage(uint64 InContainerId, FName InPackageName)
{
	check(IsInGameThread());
	if (!UE_TRACE_MINIMAL_CHANNELEXPR_IS_ENABLED(WorldStreamingDependenciesChannel) || InPackageName.IsNone() || !FPackageStore::Get().HasAnyBackendsMounted())
	{
		return;
	}

	const FPackageId PackageId = FPackageId::FromName(InPackageName);
	TSet<FPackageId> Dependencies;
	if (!FWorldStreamingPackageDependencyCache::Get().GetDependencies(PackageId, Dependencies))
	{
		UE_LOGF(LogWorldStreamingTrace, Log, "Package '%ls' - dependency lookup failed", *InPackageName.ToString());
		return;
	}

	TArray<uint64> DependencyIds;
	DependencyIds.Reserve(Dependencies.Num() + 1);
	DependencyIds.Add(PackageId.Value());
	for (const FPackageId& DependencyId : Dependencies)
	{
		DependencyIds.Add(DependencyId.Value());
	}

	TraceContainerDependencies(InContainerId, DependencyIds);
	ResolveAndEnsurePackageNamesMapped(DependencyIds);
}

void UWorldStreamingTraceSubsystem::TraceContainerDependencies(uint64 InContainerId, TArrayView<const uint64> InDependencyIds)
{
	UE_TRACE_MINIMAL_LOG(WorldStreaming, ContainerDependencies, WorldStreamingDependenciesChannel)
		<< ContainerDependencies.Id(InContainerId)
		<< ContainerDependencies.WorldId(WorldId)
		<< ContainerDependencies.DependencyIds(InDependencyIds.GetData(), InDependencyIds.Num());
}

void UWorldStreamingTraceSubsystem::ResolveAndEnsurePackageNamesMapped(TArrayView<const uint64> InPackageIds)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UWorldStreamingTraceSubsystem::ResolveAndEnsurePackageNamesMapped);

	check(IsInGameThread());

	TArray<uint64> PackageIdsNeedingNames;
	for (const uint64 PackageId : InPackageIds)
	{
		if (!KnownPackageMappings.Contains(PackageId))
		{
			KnownPackageMappings.Add(PackageId);
			PackageIdsNeedingNames.Add(PackageId);
		}
	}

	if (PackageIdsNeedingNames.IsEmpty())
	{
		return;
	}

	if (FWorldStreamingPackageIdNameMap::Get().IsReady())
	{
		ResolveAndTracePackageNames(PackageIdsNeedingNames);
	}
	else
	{
		PendingNameMappingPackageIds.Append(PackageIdsNeedingNames.GetData(), PackageIdsNeedingNames.Num());
		FWorldStreamingPackageIdNameMap::Get().BuildDatabaseAsync();
	}
}

void UWorldStreamingTraceSubsystem::ResolveAndTracePackageNames(TArrayView<const uint64> InPackageIds)
{
	check(IsInGameThread());

	for (const uint64 PackageId : InPackageIds)
	{
		FName PackageName;
		if (FWorldStreamingPackageIdNameMap::Get().GetPackageNameFromId(FPackageId::FromValue(PackageId), PackageName))
		{
			const FString PackageNameString = PackageName.ToString();
			UE_TRACE_MINIMAL_LOG(WorldStreaming, PackageNameMapping, WorldStreamingDependenciesChannel)
				<< PackageNameMapping.PackageId(PackageId)
				<< PackageNameMapping.Name(*PackageNameString, PackageNameString.Len());
		}
	}
}

void UWorldStreamingTraceSubsystem::OnNameMappingBuildComplete()
{
	check(IsInGameThread());

	if (!PendingNameMappingPackageIds.IsEmpty())
	{
		ResolveAndTracePackageNames(PendingNameMappingPackageIds);
		PendingNameMappingPackageIds.Empty();
	}
}
#endif // UE_TRACE_WORLD_STREAMING_DEPENDENCIES_ENABLED
#endif // UE_TRACE_WORLD_STREAMING_ENABLED