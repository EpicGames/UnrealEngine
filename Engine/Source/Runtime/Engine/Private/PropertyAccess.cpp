// Copyright Epic Games, Inc. All Rights Reserved.

#include "PropertyAccess.h"
#include "UObject/Stack.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PropertyAccess)

#define LOCTEXT_NAMESPACE "PropertyAccess"

namespace PropertyAccess
{
	static FCriticalSection CriticalSection;
}

struct FPropertyAccessSystem
{
	struct FResolveIndirectionsOnLoadContext
	{
		FResolveIndirectionsOnLoadContext(TArrayView<FPropertyAccessSegment> InSegments, FPropertyAccessIndirectionChain& InAccess, FPropertyAccessLibrary& InLibrary)
			: Segments(InSegments)
			, Access(InAccess)
			, Library(InLibrary)
			, AccumulatedOffset(0)
		{}

		TArrayView<FPropertyAccessSegment> Segments;
		TArray<FPropertyAccessIndirection> Indirections;
		FPropertyAccessIndirectionChain& Access;
		FPropertyAccessLibrary& Library;
		uint32 AccumulatedOffset;
		FText ErrorMessage;
	};

	static uint32 GetPropertyOffset(const FProperty* InProperty, int32 InArrayIndex = 0)
	{
		return (uint32)(InProperty->GetOffset_ForInternal() + InProperty->GetElementSize() * InArrayIndex);
	}

	// Called on load to resolve all path segments to indirections
	static bool ResolveIndirectionsOnLoad(FResolveIndirectionsOnLoadContext& InContext)
	{
		for(int32 SegmentIndex = 0; SegmentIndex < InContext.Segments.Num(); ++SegmentIndex)
		{
			const FPropertyAccessSegment& Segment = InContext.Segments[SegmentIndex];
			const bool bLastSegment = SegmentIndex == InContext.Segments.Num() - 1;

			if(Segment.Property.Get() == nullptr)
			{
				return false;
			}

			if(EnumHasAllFlags((EPropertyAccessSegmentFlags)Segment.Flags, EPropertyAccessSegmentFlags::Function))
			{
				if(Segment.Function == nullptr)
				{
					return false;
				}

				FPropertyAccessIndirection& Indirection = InContext.Indirections.AddDefaulted_GetRef();

				Indirection.Type = Segment.Function->HasAnyFunctionFlags(FUNC_Native) ? EPropertyAccessIndirectionType::NativeFunction : EPropertyAccessIndirectionType::ScriptFunction;

				switch((EPropertyAccessSegmentFlags)Segment.Flags & ~EPropertyAccessSegmentFlags::ModifierFlags)
				{
				case EPropertyAccessSegmentFlags::Struct:
				case EPropertyAccessSegmentFlags::Leaf:
					Indirection.ObjectType = EPropertyAccessObjectType::None;
					break;
				case EPropertyAccessSegmentFlags::Object:
					Indirection.ObjectType = EPropertyAccessObjectType::Object;
					break;
				case EPropertyAccessSegmentFlags::WeakObject:
					Indirection.ObjectType = EPropertyAccessObjectType::WeakObject;
					break;
				case EPropertyAccessSegmentFlags::SoftObject:
					Indirection.ObjectType = EPropertyAccessObjectType::SoftObject;
					break;
				default:
					check(false);
					break;
				}

				Indirection.Function = Segment.Function;
				Indirection.Property = Segment.Function->GetReturnProperty();
				Indirection.ReturnBufferSize = Segment.Property.Get()->GetSize();
				Indirection.ReturnBufferAlignment = Segment.Property.Get()->GetMinAlignment();
			}
			else
			{
				const int32 ArrayIndex = Segment.ArrayIndex == INDEX_NONE ? 0 : Segment.ArrayIndex;
				const EPropertyAccessSegmentFlags UnmodifiedFlags = (EPropertyAccessSegmentFlags)Segment.Flags & ~EPropertyAccessSegmentFlags::ModifierFlags;
				switch(UnmodifiedFlags)
				{
				case EPropertyAccessSegmentFlags::Struct:
				case EPropertyAccessSegmentFlags::Leaf:
				{
					FPropertyAccessIndirection& Indirection = InContext.Indirections.AddDefaulted_GetRef();

					Indirection.Offset = GetPropertyOffset(Segment.Property.Get(), ArrayIndex);
					Indirection.Type = EPropertyAccessIndirectionType::Offset;
					break;
				}
				case EPropertyAccessSegmentFlags::Object:
				{
					FPropertyAccessIndirection& Indirection = InContext.Indirections.AddDefaulted_GetRef();

					Indirection.Offset = GetPropertyOffset(Segment.Property.Get(), ArrayIndex);
					Indirection.Type = EPropertyAccessIndirectionType::Object;
					Indirection.ObjectType = EPropertyAccessObjectType::Object;
					break;
				}
				case EPropertyAccessSegmentFlags::WeakObject:
				{
					FPropertyAccessIndirection& Indirection = InContext.Indirections.AddDefaulted_GetRef();

					Indirection.Offset = GetPropertyOffset(Segment.Property.Get(), ArrayIndex);
					Indirection.Type = EPropertyAccessIndirectionType::Object;
					Indirection.ObjectType = EPropertyAccessObjectType::WeakObject;
					break;
				}
				case EPropertyAccessSegmentFlags::SoftObject:
				{
					FPropertyAccessIndirection& Indirection = InContext.Indirections.AddDefaulted_GetRef();

					Indirection.Offset = GetPropertyOffset(Segment.Property.Get(), ArrayIndex);
					Indirection.Type = EPropertyAccessIndirectionType::Object;
					Indirection.ObjectType = EPropertyAccessObjectType::SoftObject;
					break;
				}
				case EPropertyAccessSegmentFlags::Array:
				case EPropertyAccessSegmentFlags::ArrayOfStructs:
				case EPropertyAccessSegmentFlags::ArrayOfObjects:
				{
					FPropertyAccessIndirection& Indirection = InContext.Indirections.AddDefaulted_GetRef();

					Indirection.Offset = GetPropertyOffset(Segment.Property.Get());
					Indirection.Type = EPropertyAccessIndirectionType::Array;
					Indirection.Property = CastFieldChecked<FArrayProperty>(Segment.Property.Get());
					Indirection.ArrayIndex = ArrayIndex;
					if(UnmodifiedFlags == EPropertyAccessSegmentFlags::ArrayOfObjects)
					{
						if(!bLastSegment)
						{
							// Object arrays need an object dereference adding if non-leaf
							FPropertyAccessIndirection& ExtraIndirection = InContext.Indirections.AddDefaulted_GetRef();

							ExtraIndirection.Offset = 0;
							ExtraIndirection.Type = EPropertyAccessIndirectionType::Object;

							FProperty* InnerProperty = CastFieldChecked<FArrayProperty>(Indirection.Property.Get())->Inner;
							if(FObjectProperty* ObjectProperty = CastField<FObjectProperty>(InnerProperty))
							{
								ExtraIndirection.ObjectType = EPropertyAccessObjectType::Object;
							}
							else if(FWeakObjectProperty* WeakObjectProperty = CastField<FWeakObjectProperty>(InnerProperty))
							{
								ExtraIndirection.ObjectType = EPropertyAccessObjectType::WeakObject;
							}
							else if(FSoftObjectProperty* SoftObjectProperty = CastField<FSoftObjectProperty>(InnerProperty))
							{
								ExtraIndirection.ObjectType = EPropertyAccessObjectType::SoftObject;
							}
						}
					}
					break;
				}
				default:
					check(false);
					break;
				}
			}
		}

		// Collapse adjacent offset indirections
		for(int32 IndirectionIndex = 0; IndirectionIndex < InContext.Indirections.Num(); ++IndirectionIndex)
		{
			FPropertyAccessIndirection& StartIndirection = InContext.Indirections[IndirectionIndex];
			if(StartIndirection.Type == EPropertyAccessIndirectionType::Offset)
			{
				for(int32 NextIndirectionIndex = IndirectionIndex + 1; NextIndirectionIndex < InContext.Indirections.Num(); ++NextIndirectionIndex)
				{
					FPropertyAccessIndirection& RunIndirection = InContext.Indirections[NextIndirectionIndex];
					if(RunIndirection.Type == EPropertyAccessIndirectionType::Offset)
					{
						StartIndirection.Offset += RunIndirection.Offset;
						InContext.Indirections.RemoveAt(NextIndirectionIndex);
					}
					else
					{
						// No run, exit
						break;
					}
				}
			}
		}

		// Concatenate indirections into the library and update the access
		InContext.Access.IndirectionStartIndex = InContext.Library.Indirections.Num();
		InContext.Library.Indirections.Append(InContext.Indirections);
		InContext.Access.IndirectionEndIndex = InContext.Library.Indirections.Num();

		// Copy leaf property to access
		FPropertyAccessSegment& LastSegment = InContext.Segments.Last();
		switch((EPropertyAccessSegmentFlags)LastSegment.Flags & ~EPropertyAccessSegmentFlags::ModifierFlags)
		{
		case EPropertyAccessSegmentFlags::Struct:
		case EPropertyAccessSegmentFlags::Leaf:
		case EPropertyAccessSegmentFlags::Object:
		case EPropertyAccessSegmentFlags::WeakObject:
		case EPropertyAccessSegmentFlags::SoftObject:
			InContext.Access.Property = LastSegment.Property.Get();
			break;
		case EPropertyAccessSegmentFlags::Array:
		case EPropertyAccessSegmentFlags::ArrayOfStructs:
		case EPropertyAccessSegmentFlags::ArrayOfObjects:
			InContext.Access.Property = CastFieldChecked<FArrayProperty>(LastSegment.Property.Get())->Inner;
			break;
		}

		return true;
	}

	static void PatchPropertyOffsets(FPropertyAccessLibrary& InLibrary)
	{
		// Need to perform a lock as copying can race with PatchPropertyOffsets in async loading thread-enabled builds
		FScopeLock Lock(&PropertyAccess::CriticalSection);

		InLibrary.Indirections.Reset();

		const int32 SrcCount = InLibrary.SrcPaths.Num();
		InLibrary.SrcAccesses.Reset();
		InLibrary.SrcAccesses.SetNum(SrcCount);
		const int32 DestCount = InLibrary.DestPaths.Num();
		InLibrary.DestAccesses.Reset();
		InLibrary.DestAccesses.SetNum(DestCount);

		// @TODO: ParallelFor this if required

		for(int32 SrcIndex = 0; SrcIndex < SrcCount; ++SrcIndex)
		{
			TArrayView<FPropertyAccessSegment> Segments(&InLibrary.PathSegments[InLibrary.SrcPaths[SrcIndex].PathSegmentStartIndex], InLibrary.SrcPaths[SrcIndex].PathSegmentCount);
			FResolveIndirectionsOnLoadContext Context(Segments, InLibrary.SrcAccesses[SrcIndex], InLibrary);
			if(!ResolveIndirectionsOnLoad(Context))
			{
				Context.Indirections.Empty();
				Context.Access.IndirectionStartIndex = Context.Access.IndirectionEndIndex = INDEX_NONE;
			}
		}

		for(int32 DestIndex = 0; DestIndex < DestCount; ++DestIndex)
		{
			TArrayView<FPropertyAccessSegment> Segments(&InLibrary.PathSegments[InLibrary.DestPaths[DestIndex].PathSegmentStartIndex], InLibrary.DestPaths[DestIndex].PathSegmentCount);
			FResolveIndirectionsOnLoadContext Context(Segments, InLibrary.DestAccesses[DestIndex], InLibrary);
			if(!ResolveIndirectionsOnLoad(Context))
			{
				Context.Indirections.Empty();
				Context.Access.IndirectionStartIndex = Context.Access.IndirectionEndIndex = INDEX_NONE;
			}
		}

		InLibrary.bHasBeenPostLoaded = true;
	}

	static void PerformCopy(const FPropertyAccessCopy& Copy, const FProperty* InSrcProperty, const void* InSrcAddr, const FProperty* InDestProperty, void* InDestAddr, TFunctionRef<void(const FProperty*, void*)> InPostCopyOperation)
	{
		switch(Copy.Type)
		{
		case EPropertyAccessCopyType::Plain:
			checkSlow(InSrcProperty->PropertyFlags & CPF_IsPlainOldData);
			checkSlow(InDestProperty->PropertyFlags & CPF_IsPlainOldData);
			FMemory::Memcpy(InDestAddr, InSrcAddr, InSrcProperty->GetElementSize());
			break;
		case EPropertyAccessCopyType::Complex:
			InSrcProperty->CopyCompleteValue(InDestAddr, InSrcAddr);
			break;
		case EPropertyAccessCopyType::Bool:
			checkSlow(InSrcProperty->IsA<FBoolProperty>());
			checkSlow(InDestProperty->IsA<FBoolProperty>());
			static_cast<const FBoolProperty*>(InDestProperty)->SetPropertyValue(InDestAddr, static_cast<const FBoolProperty*>(InSrcProperty)->GetPropertyValue(InSrcAddr));
			break;
		case EPropertyAccessCopyType::Struct:
			checkSlow(InSrcProperty->IsA<FStructProperty>());
			checkSlow(InDestProperty->IsA<FStructProperty>());
			static_cast<const FStructProperty*>(InDestProperty)->Struct->CopyScriptStruct(InDestAddr, InSrcAddr);
			break;
		case EPropertyAccessCopyType::Object:
			checkSlow(InSrcProperty->IsA<FObjectPropertyBase>());
			checkSlow(InDestProperty->IsA<FObjectPropertyBase>());
			static_cast<const FObjectPropertyBase*>(InDestProperty)->SetObjectPropertyValue(InDestAddr, static_cast<const FObjectPropertyBase*>(InSrcProperty)->GetObjectPropertyValue(InSrcAddr));
			break;
		case EPropertyAccessCopyType::Name:
			checkSlow(InSrcProperty->IsA<FNameProperty>());
			checkSlow(InDestProperty->IsA<FNameProperty>());
			static_cast<const FNameProperty*>(InDestProperty)->SetPropertyValue(InDestAddr, static_cast<const FNameProperty*>(InSrcProperty)->GetPropertyValue(InSrcAddr));
			break;
		case EPropertyAccessCopyType::Array:
		{
			checkSlow(InSrcProperty->IsA<FArrayProperty>());
			checkSlow(InDestProperty->IsA<FArrayProperty>());
			const FArrayProperty* SrcArrayProperty = static_cast<const FArrayProperty*>(InSrcProperty);
			const FArrayProperty* DestArrayProperty = static_cast<const FArrayProperty*>(InDestProperty);
			FScriptArrayHelper SourceArrayHelper(SrcArrayProperty, InSrcAddr);
			FScriptArrayHelper DestArrayHelper(DestArrayProperty, InDestAddr);

			// Copy the minimum number of elements to the destination array without resizing
			const int32 MinSize = FMath::Min(SourceArrayHelper.Num(), DestArrayHelper.Num());
			for(int32 ElementIndex = 0; ElementIndex < MinSize; ++ElementIndex)
			{
				SrcArrayProperty->Inner->CopySingleValue(DestArrayHelper.GetRawPtr(ElementIndex), SourceArrayHelper.GetRawPtr(ElementIndex));
			}
			break;
		}
		case EPropertyAccessCopyType::PromoteBoolToByte:
			checkSlow(InSrcProperty->IsA<FBoolProperty>());
			checkSlow(InDestProperty->IsA<FByteProperty>());
			static_cast<const FByteProperty*>(InDestProperty)->SetPropertyValue(InDestAddr, (uint8)static_cast<const FBoolProperty*>(InSrcProperty)->GetPropertyValue(InSrcAddr));
			break;
		case EPropertyAccessCopyType::PromoteBoolToInt32:
			checkSlow(InSrcProperty->IsA<FBoolProperty>());
			checkSlow(InDestProperty->IsA<FIntProperty>());
			static_cast<const FIntProperty*>(InDestProperty)->SetPropertyValue(InDestAddr, (int32)static_cast<const FBoolProperty*>(InSrcProperty)->GetPropertyValue(InSrcAddr));
			break;
		case EPropertyAccessCopyType::PromoteBoolToInt64:
			checkSlow(InSrcProperty->IsA<FBoolProperty>());
			checkSlow(InDestProperty->IsA<FInt64Property>());
			static_cast<const FInt64Property*>(InDestProperty)->SetPropertyValue(InDestAddr, (int64)static_cast<const FBoolProperty*>(InSrcProperty)->GetPropertyValue(InSrcAddr));
			break;
		case EPropertyAccessCopyType::PromoteBoolToFloat:
			checkSlow(InSrcProperty->IsA<FBoolProperty>());
			checkSlow(InDestProperty->IsA<FFloatProperty>());
			static_cast<const FFloatProperty*>(InDestProperty)->SetPropertyValue(InDestAddr, (float)static_cast<const FBoolProperty*>(InSrcProperty)->GetPropertyValue(InSrcAddr));
			break;
		case EPropertyAccessCopyType::PromoteBoolToDouble:
			checkSlow(InSrcProperty->IsA<FBoolProperty>());
			checkSlow(InDestProperty->IsA<FDoubleProperty>());
			static_cast<const FDoubleProperty*>(InDestProperty)->SetPropertyValue(InDestAddr, (double)static_cast<const FBoolProperty*>(InSrcProperty)->GetPropertyValue(InSrcAddr));
			break;
		case EPropertyAccessCopyType::PromoteByteToInt32:
			checkSlow(InSrcProperty->IsA<FByteProperty>());
			checkSlow(InDestProperty->IsA<FIntProperty>());
			static_cast<const FIntProperty*>(InDestProperty)->SetPropertyValue(InDestAddr, (int32)static_cast<const FByteProperty*>(InSrcProperty)->GetPropertyValue(InSrcAddr));
			break;
		case EPropertyAccessCopyType::PromoteByteToInt64:
			checkSlow(InSrcProperty->IsA<FByteProperty>());
			checkSlow(InDestProperty->IsA<FInt64Property>());
			static_cast<const FInt64Property*>(InDestProperty)->SetPropertyValue(InDestAddr, (int64)static_cast<const FByteProperty*>(InSrcProperty)->GetPropertyValue(InSrcAddr));
			break;
		case EPropertyAccessCopyType::PromoteByteToFloat:
			checkSlow(InSrcProperty->IsA<FByteProperty>());
			checkSlow(InDestProperty->IsA<FFloatProperty>());
			static_cast<const FFloatProperty*>(InDestProperty)->SetPropertyValue(InDestAddr, (float)static_cast<const FByteProperty*>(InSrcProperty)->GetPropertyValue(InSrcAddr));
			break;
		case EPropertyAccessCopyType::PromoteByteToDouble:
			checkSlow(InSrcProperty->IsA<FByteProperty>());
			checkSlow(InDestProperty->IsA<FDoubleProperty>());
			static_cast<const FDoubleProperty*>(InDestProperty)->SetPropertyValue(InDestAddr, (double)static_cast<const FByteProperty*>(InSrcProperty)->GetPropertyValue(InSrcAddr));
			break;
		case EPropertyAccessCopyType::PromoteInt32ToInt64:
			checkSlow(InSrcProperty->IsA<FIntProperty>());
			checkSlow(InDestProperty->IsA<FInt64Property>());
			static_cast<const FInt64Property*>(InDestProperty)->SetPropertyValue(InDestAddr, (int64)static_cast<const FIntProperty*>(InSrcProperty)->GetPropertyValue(InSrcAddr));
			break;
		case EPropertyAccessCopyType::PromoteInt32ToFloat:
			checkSlow(InSrcProperty->IsA<FIntProperty>());
			checkSlow(InDestProperty->IsA<FFloatProperty>());
			static_cast<const FFloatProperty*>(InDestProperty)->SetPropertyValue(InDestAddr, (float)static_cast<const FIntProperty*>(InSrcProperty)->GetPropertyValue(InSrcAddr));
			break;
		case EPropertyAccessCopyType::PromoteInt32ToDouble:
			checkSlow(InSrcProperty->IsA<FIntProperty>());
			checkSlow(InDestProperty->IsA<FDoubleProperty>());
			static_cast<const FDoubleProperty*>(InDestProperty)->SetPropertyValue(InDestAddr, (double)static_cast<const FIntProperty*>(InSrcProperty)->GetPropertyValue(InSrcAddr));
			break;
		case EPropertyAccessCopyType::PromoteFloatToDouble:
			checkSlow(InSrcProperty->IsA<FFloatProperty>());
			checkSlow(InDestProperty->IsA<FDoubleProperty>());
			static_cast<const FDoubleProperty*>(InDestProperty)->SetPropertyValue(InDestAddr, (double)static_cast<const FFloatProperty*>(InSrcProperty)->GetPropertyValue(InSrcAddr));
			break;
		case EPropertyAccessCopyType::DemoteDoubleToFloat:
			checkSlow(InSrcProperty->IsA<FDoubleProperty>());
			checkSlow(InDestProperty->IsA<FFloatProperty>());
			static_cast<const FFloatProperty*>(InDestProperty)->SetPropertyValue(InDestAddr, (float)static_cast<const FDoubleProperty*>(InSrcProperty)->GetPropertyValue(InSrcAddr));
			break;
		case EPropertyAccessCopyType::PromoteArrayFloatToDouble:
		{
			checkSlow(InSrcProperty->IsA<FArrayProperty>());
			checkSlow(InDestProperty->IsA<FArrayProperty>());
			const FArrayProperty* SrcArrayProperty = ExactCastField<const FArrayProperty>(InSrcProperty);
			const FArrayProperty* DestArrayProperty = ExactCastField<const FArrayProperty>(InDestProperty);
			CopyAndCastFloatingPointArray<float, double>(SrcArrayProperty, InSrcAddr, DestArrayProperty, InDestAddr);
			break;
		}
		case EPropertyAccessCopyType::DemoteArrayDoubleToFloat:
		{
			checkSlow(InSrcProperty->IsA<FArrayProperty>());
			checkSlow(InDestProperty->IsA<FArrayProperty>());
			const FArrayProperty* SrcArrayProperty = ExactCastField<const FArrayProperty>(InSrcProperty);
			const FArrayProperty* DestArrayProperty = ExactCastField<const FArrayProperty>(InDestProperty);
			CopyAndCastFloatingPointArray<double, float>(SrcArrayProperty, InSrcAddr, DestArrayProperty, InDestAddr);
			break;
		}
		case EPropertyAccessCopyType::PromoteMapValueFloatToDouble:
			{
				checkSlow(InSrcProperty->IsA<FMapProperty>());
				checkSlow(InDestProperty->IsA<FMapProperty>());
				const FMapProperty* SrcMapProperty = ExactCastField<const FMapProperty>(InSrcProperty);
				const FMapProperty* DestMapProperty = ExactCastField<const FMapProperty>(InDestProperty);
				CopyAndCastFloatingPointMapValues<float, double>(SrcMapProperty, InSrcAddr, DestMapProperty, InDestAddr);
				break;
			}
		case EPropertyAccessCopyType::DemoteMapValueDoubleToFloat:
			{
				checkSlow(InSrcProperty->IsA<FMapProperty>());
				checkSlow(InDestProperty->IsA<FMapProperty>());
				const FMapProperty* SrcMapProperty = ExactCastField<const FMapProperty>(InSrcProperty);
				const FMapProperty* DestMapProperty = ExactCastField<const FMapProperty>(InDestProperty);
				CopyAndCastFloatingPointMapValues<double, float>(SrcMapProperty, InSrcAddr, DestMapProperty, InDestAddr);
				break;
			}	
		default:
			check(false);
			break;
		}

		InPostCopyOperation(InDestProperty, InDestAddr);
	}

	static void CallNativeAccessor(UObject* InObject, UFunction* InFunction, void* OutRetValue)
	{
		// Func must be local
		check((InObject->GetFunctionCallspace(InFunction, nullptr) & FunctionCallspace::Local) != 0);		

		// Function must be native
		check(InFunction->HasAnyFunctionFlags(FUNC_Native));

		// Function must have a return property
		check(InFunction->GetReturnProperty() != nullptr);

		// Function must only have one param - its return value
		check(InFunction->NumParms == 1);

		FFrame Stack(InObject, InFunction, nullptr, nullptr, InFunction->ChildProperties);
		InFunction->Invoke(InObject, Stack, OutRetValue);
	}

	// Gets the address that corresponds to a property access.
	// Forwards the address onto the passed-in function. This callback-style approach is used because in some cases 
	// (e.g. functions), the address may be memory allocated on the stack.
	template<typename PredicateType>
	static void GetAccessAddress(void* InContainer, const FPropertyAccessLibrary& InLibrary, const FPropertyAccessIndirectionChain& InAccess, PredicateType InAddressFunction)
	{
		// Buffer for function return values
		TArray<uint8, TNonRelocatableInlineAllocator<64>> ArrayBuffer;
 
		// Property that we initialize the buffer with
		FProperty* BufferProperty = nullptr;
 
		void* Address = InContainer;
 
		for(int32 IndirectionIndex = InAccess.IndirectionStartIndex; Address != nullptr && IndirectionIndex < InAccess.IndirectionEndIndex; ++IndirectionIndex)
		{
			const FPropertyAccessIndirection& Indirection = InLibrary.Indirections[IndirectionIndex];
 
			switch(Indirection.Type)
			{
			case EPropertyAccessIndirectionType::Offset:
				Address = static_cast<void*>(static_cast<uint8*>(Address) + Indirection.Offset);
				break;
			case EPropertyAccessIndirectionType::Object:
			{
				switch(Indirection.ObjectType)
				{
				case EPropertyAccessObjectType::Object:
				{
					UObject* Object = *reinterpret_cast<UObject**>(static_cast<uint8*>(Address) + Indirection.Offset);
					Address = static_cast<void*>(Object);
					break;
				}
				case EPropertyAccessObjectType::WeakObject:
				{
					TWeakObjectPtr<UObject>& WeakObjectPtr = *reinterpret_cast<TWeakObjectPtr<UObject>*>(static_cast<uint8*>(Address) + Indirection.Offset);
					UObject* Object = WeakObjectPtr.Get();
					Address = static_cast<void*>(Object);
					break;
				}
				case EPropertyAccessObjectType::SoftObject:
				{
					FSoftObjectPtr& SoftObjectPtr = *reinterpret_cast<FSoftObjectPtr*>(static_cast<uint8*>(Address) + Indirection.Offset);
					UObject* Object = SoftObjectPtr.Get();
					Address = static_cast<void*>(Object);
					break;
				}
				default:
					Address = nullptr;
					checkNoEntry();
					break;
				}
				break;
			}
			case EPropertyAccessIndirectionType::Array:
			{
				if(FArrayProperty* ArrayProperty = CastFieldChecked<FArrayProperty>(Indirection.Property.Get()))
				{
					FScriptArrayHelper Helper(ArrayProperty, static_cast<uint8*>(Address) + Indirection.Offset);
					if(Helper.IsValidIndex(Indirection.ArrayIndex))
					{
						Address = static_cast<void*>(Helper.GetRawPtr(Indirection.ArrayIndex));
					}
					else
					{
						Address = nullptr;
					}
				}
				else
				{
					Address = nullptr;
				}
				break;
			}
			case EPropertyAccessIndirectionType::ScriptFunction:
			case EPropertyAccessIndirectionType::NativeFunction:
			{
				if(Indirection.Function != nullptr && Indirection.Function->GetReturnProperty() != nullptr)
				{
					UObject* CalleeObject = static_cast<UObject*>(Address);
 
					// Allocate buffer + alignment slack for the return value
					ArrayBuffer.SetNumUninitialized(Indirection.ReturnBufferSize + Indirection.ReturnBufferAlignment, EAllowShrinking::No);
					Address = Align(ArrayBuffer.GetData(), Indirection.ReturnBufferAlignment);
 
					// Init value
					check(Indirection.Function->GetReturnProperty() == Indirection.Property.Get());
					Indirection.Property.Get()->InitializeValue(Address);
					BufferProperty = Indirection.Property.Get();
 
					if(Indirection.Type == EPropertyAccessIndirectionType::NativeFunction)
					{
						CallNativeAccessor(CalleeObject, Indirection.Function, Address);
					}
					else
					{
						CalleeObject->ProcessEvent(Indirection.Function, Address);
					}
 
					// Function access may return an object, so we need to follow that ptr
					switch(Indirection.ObjectType)
					{
					case EPropertyAccessObjectType::Object:
					{
						UObject* Object = *static_cast<UObject**>(Address);
						Address = static_cast<void*>(Object);
						break;
					}
					case EPropertyAccessObjectType::WeakObject:
					{
						TWeakObjectPtr<UObject>& WeakObjectPtr = *static_cast<TWeakObjectPtr<UObject>*>(Address);
						UObject* Object = WeakObjectPtr.Get();
						Address = static_cast<void*>(Object);
						break;
					}
					case EPropertyAccessObjectType::SoftObject:
					{
						FSoftObjectPtr& SoftObjectPtr = *reinterpret_cast<FSoftObjectPtr*>(Address);
						UObject* Object = SoftObjectPtr.Get();
						Address = static_cast<void*>(Object);
						break;
					}
					default:
						break;
					}
				}
				else
				{
					Address = nullptr;
				}
				break;
			}
			default:
				Address = nullptr;
				checkNoEntry();
				break;
			}
		}
 
		if(Address != nullptr)
		{
			InAddressFunction(Address);
 
			if (BufferProperty != nullptr)
			{
				BufferProperty->DestroyValue(Address);
			}
		}
	}
	
	static void GetAccessAddress(void* InContainer, const FPropertyAccessLibrary& InLibrary, int32 InAccessIndex, TFunctionRef<void(const FProperty*, void*)> InFunction)
	{
		if(InLibrary.DestAccesses.IsValidIndex(InAccessIndex))
		{
			const FPropertyAccessIndirectionChain& DestAccess = InLibrary.DestAccesses[InAccessIndex];
			if (FProperty* DestProperty = DestAccess.Property.Get())
			{
				GetAccessAddress(InContainer, InLibrary, DestAccess, [&InFunction, DestProperty](void* InAddress)
				{
					InFunction(DestProperty, InAddress);
				});
			}
		}
	}

	// Process a single copy
	static void ProcessCopy(UStruct* InStruct, void* InContainer, const FPropertyAccessLibrary& InLibrary, int32 InCopyIndex, int32 InBatchId, TFunctionRef<void(const FProperty*, void*)> InPostCopyOperation)
	{
		if(InLibrary.bHasBeenPostLoaded)
		{
			if(InLibrary.CopyBatchArray.IsValidIndex(InBatchId))
			{
				const FPropertyAccessCopy& Copy = InLibrary.CopyBatchArray[InBatchId].Copies[InCopyIndex];
				const FPropertyAccessIndirectionChain& SrcAccess = InLibrary.SrcAccesses[Copy.AccessIndex];
				if(FProperty* SourceProperty = SrcAccess.Property.Get())
				{
					GetAccessAddress(InContainer, InLibrary, SrcAccess, [InContainer, &InLibrary, &Copy, SourceProperty, &InPostCopyOperation](void* InSrcAddress)
					{
						for(int32 DestAccessIndex = Copy.DestAccessStartIndex; DestAccessIndex < Copy.DestAccessEndIndex; ++DestAccessIndex)
						{
							const FPropertyAccessIndirectionChain& DestAccess = InLibrary.DestAccesses[DestAccessIndex];
							if(FProperty* DestProperty = DestAccess.Property.Get())
							{
								GetAccessAddress(InContainer, InLibrary, DestAccess, [&InSrcAddress, &Copy, SourceProperty, DestProperty, &InPostCopyOperation](void* InDestAddress)
								{
									PerformCopy(Copy, SourceProperty, InSrcAddress, DestProperty, InDestAddress, InPostCopyOperation);
								});
							}
						}
					});
				}
			}
		}
	}

	static void ProcessCopies(UStruct* InStruct, void* InContainer, const FPropertyAccessLibrary& InLibrary, int32 InBatchId)
	{
		if(InLibrary.bHasBeenPostLoaded)
		{
			if(InLibrary.CopyBatchArray.IsValidIndex(InBatchId))
			{
				// Copy all valid properties
				// Parallelization opportunity: ParallelFor all the property copies we need to make
				const int32 NumCopies = InLibrary.CopyBatchArray[InBatchId].Copies.Num();
				for(int32 CopyIndex = 0; CopyIndex < NumCopies; ++CopyIndex)
				{
					ProcessCopy(InStruct, InContainer, InLibrary, CopyIndex, InBatchId, [](const FProperty*, void*){});
				}
			}
		}
	}

	template <typename SourceType, typename DestinationType>
	static void CopyAndCastFloatingPointArray(const FArrayProperty* SourceArrayProperty,
											  const void* SourceAddress,
											  const FArrayProperty* DestinationArrayProperty,
											  void* DestinationAddress)
	{
		checkSlow(SourceArrayProperty);
		checkSlow(SourceAddress);
		checkSlow(DestinationArrayProperty);
		checkSlow(DestinationAddress);

		FScriptArrayHelper SourceArrayHelper(SourceArrayProperty, SourceAddress);
		FScriptArrayHelper DestinationArrayHelper(DestinationArrayProperty, DestinationAddress);

		DestinationArrayHelper.Resize(SourceArrayHelper.Num());
		for (int32 i = 0; i < SourceArrayHelper.Num(); ++i)
		{
			const SourceType* SourceData = reinterpret_cast<const SourceType*>(SourceArrayHelper.GetRawPtr(i));
			DestinationType* DestinationData = reinterpret_cast<DestinationType*>(DestinationArrayHelper.GetRawPtr(i));

			*DestinationData = static_cast<DestinationType>(*SourceData);
		}
	}

	template <typename SourceType, typename DestinationType>
	static void CopyAndCastFloatingPointMapValues(const FMapProperty* SourceMapProperty,
												  const void* SourceAddress,
												  const FMapProperty* DestinationMapProperty,
												  void* DestinationAddress)
	{
		checkSlow(SourceMapProperty);
		checkSlow(SourceAddress);
		checkSlow(DestinationMapProperty);
		checkSlow(DestinationAddress);

		FScriptMapHelper SourceMapHelper(SourceMapProperty, SourceAddress);
		FScriptMapHelper DestinationMapHelper(DestinationMapProperty, DestinationAddress);

		DestinationMapHelper.EmptyValues();
		for (FScriptMapHelper::FIterator It(SourceMapHelper); It; ++It)
		{
			const void* KeyData = SourceMapHelper.GetKeyPtr(It);
			const SourceType* SourceValueData = reinterpret_cast<const SourceType*>(SourceMapHelper.GetValuePtr(It));
			DestinationType CastedType = static_cast<DestinationType>(*SourceValueData);
			DestinationMapHelper.AddPair(KeyData, &CastedType);
		}
	}	
};

// Unlike the copy assignment operator, the copy constructor doesn't take a lock
FPropertyAccessLibrary::FPropertyAccessLibrary(const FPropertyAccessLibrary& Other) = default;

const FPropertyAccessLibrary& FPropertyAccessLibrary::operator =(const FPropertyAccessLibrary& Other)
{
	// Need to perform a lock as copying can race with PatchPropertyOffsets in async loading thread-enabled builds
	FScopeLock Lock(&PropertyAccess::CriticalSection);

	PathSegments = Other.PathSegments;
	SrcPaths = Other.SrcPaths;
	DestPaths = Other.DestPaths;
	CopyBatchArray = Other.CopyBatchArray;
	SrcAccesses = Other.SrcAccesses;
	DestAccesses = Other.DestAccesses;
	Indirections = Other.Indirections;
	bHasBeenPostLoaded = Other.bHasBeenPostLoaded;
	
	return *this;
}

namespace PropertyAccess
{
	void PostLoadLibrary(FPropertyAccessLibrary& InLibrary)
	{
		::FPropertyAccessSystem::PatchPropertyOffsets(InLibrary);
	}

	void PatchPropertyOffsets(FPropertyAccessLibrary& InLibrary)
	{
		::FPropertyAccessSystem::PatchPropertyOffsets(InLibrary);
	}

	void ProcessCopies(UObject* InObject, const FPropertyAccessLibrary& InLibrary, EPropertyAccessCopyBatch InBatchType)
	{
		ProcessCopies(InObject, InLibrary, FCopyBatchId((__underlying_type(EPropertyAccessCopyBatch))InBatchType));
	}

	void ProcessCopies(UObject* InObject, const FPropertyAccessLibrary& InLibrary, const FCopyBatchId& InBatchId)
	{
		::FPropertyAccessSystem::ProcessCopies(InObject->GetClass(), InObject, InLibrary, InBatchId.Id);
	}

	void ProcessCopy(UObject* InObject, const FPropertyAccessLibrary& InLibrary, EPropertyAccessCopyBatch InBatchType, int32 InCopyIndex, TFunctionRef<void(const FProperty*, void*)> InPostCopyOperation)
	{
		ProcessCopy(InObject, InLibrary, FCopyBatchId((__underlying_type(EPropertyAccessCopyBatch))InBatchType), InCopyIndex, InPostCopyOperation);
	}

	void ProcessCopy(UObject* InObject, const FPropertyAccessLibrary& InLibrary, const FCopyBatchId& InBatchId, int32 InCopyIndex, TFunctionRef<void(const FProperty*, void*)> InPostCopyOperation)
	{
		::FPropertyAccessSystem::ProcessCopy(InObject->GetClass(), InObject, InLibrary, InCopyIndex, InBatchId.Id, InPostCopyOperation);
	}

	void GetAccessAddress(UObject* InObject, const FPropertyAccessLibrary& InLibrary, int32 InAccessIndex, TFunctionRef<void(const FProperty*, void*)> InFunction)
	{
		::FPropertyAccessSystem::GetAccessAddress(InObject, InLibrary, InAccessIndex, InFunction);
	}

	void BindEvents(UObject* InObject, const FPropertyAccessLibrary& InLibrary)
	{
	}

	int32 GetEventId(const UClass* InClass, TArrayView<const FName> InPath)
	{
		return INDEX_NONE;
	}
}

#undef LOCTEXT_NAMESPACE
