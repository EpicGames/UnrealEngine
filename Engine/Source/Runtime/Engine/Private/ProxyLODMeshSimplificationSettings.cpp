// Copyright Epic Games, Inc. All Rights Reserved.

#include "Engine/ProxyLODMeshSimplificationSettings.h"
#include "HAL/IConsoleManager.h"
#include "CoreGlobals.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ProxyLODMeshSimplificationSettings)

static FAutoConsoleVariable CVarProxyLODMeshReductionModule(
	TEXT("r.ProxyLODMeshReductionModule"),
	TEXT("ProxyLODMeshReduction"),
	TEXT("Name of the Proxy LOD reduction module to choose. If blank it chooses any that exist.\n"),
	ECVF_ReadOnly);

UProxyLODMeshSimplificationSettings::UProxyLODMeshSimplificationSettings(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

FName UProxyLODMeshSimplificationSettings::GetContainerName() const
{
	static const FName ContainerName("Project");
	return ContainerName;
}

FName UProxyLODMeshSimplificationSettings::GetCategoryName() const
{
	static const FName EditorCategoryName("Editor");
	return EditorCategoryName;
}

void UProxyLODMeshSimplificationSettings::SetProxyLODMeshReductionModuleName(FName InProxyLODMeshReductionModuleName)
{
	ProxyLODMeshReductionModuleName = InProxyLODMeshReductionModuleName;
	CVarProxyLODMeshReductionModule->Set(*ProxyLODMeshReductionModuleName.ToString());
}

void UProxyLODMeshSimplificationSettings::PostInitProperties()
{
	Super::PostInitProperties();

#if WITH_EDITOR
	if (IsTemplate())
	{
		ImportConsoleVariableValues();
	}
#endif
}

#if WITH_EDITOR


void UProxyLODMeshSimplificationSettings::PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	if (PropertyChangedEvent.Property)
	{
		ExportValuesToConsoleVariables(PropertyChangedEvent.Property);
	}
}


#endif

