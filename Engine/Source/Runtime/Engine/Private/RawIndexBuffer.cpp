// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	RawIndexBuffer.cpp: Raw index buffer implementation.
=============================================================================*/

#include "RawIndexBuffer.h"

#include "DataDrivenShaderPlatformInfo.h"
#include "Interfaces/ITargetPlatform.h"
#include "RenderUtils.h"
#include "Modules/ModuleManager.h"
#include "RenderingThread.h"
#include "RHIResourceReplace.h"

#if WITH_EDITOR
#include "MeshUtilities.h"
template<typename IndexDataType, typename Allocator>
void CacheOptimizeIndexBuffer(TArray<IndexDataType,Allocator>& Indices)
{
	TArray<IndexDataType> TempIndices(Indices);
	IMeshUtilities& MeshUtilities = FModuleManager::LoadModuleChecked<IMeshUtilities>("MeshUtilities");
	MeshUtilities.CacheOptimizeIndexBuffer(TempIndices);
	Indices = TempIndices;
}
#endif // #if WITH_EDITOR

/**
 * Whether buffers should have ShaderResource usage flag, so that systems can optionally opt in to create SRVs for them 
 */
static bool AreIndexBuffersShaderResources(EShaderPlatform ShaderPlatform)
{
	return AreBufferSRVsAlwaysCreatedByDefault(ShaderPlatform) || IsGPUSkinPassThroughSupported(ShaderPlatform);
} 

/*-----------------------------------------------------------------------------
FRawIndexBuffer
-----------------------------------------------------------------------------*/

/**
* Orders a triangle list for better vertex cache coherency.
*/
void FRawIndexBuffer::CacheOptimize()
{
#if WITH_EDITOR
	CacheOptimizeIndexBuffer(Indices);
#endif
}

void FRawIndexBuffer::InitRHI(FRHICommandListBase& RHICmdList)
{
	uint32 Size = Indices.Num() * sizeof(uint16);
	if (Indices.Num() > 0)
	{
		// Create the index buffer.
		const FRHIBufferCreateDesc CreateDesc =
			FRHIBufferCreateDesc::CreateIndex<uint16>(TEXT("FRawIndexBuffer"), Indices.Num())
			.AddUsage(EBufferUsageFlags::Static)
			.SetInitialState(ERHIAccess::VertexOrIndexBuffer | ERHIAccess::SRVMask)
			.SetInitActionInitializer();

		TRHIBufferInitializer<uint16> Initializer = RHICmdList.CreateBufferInitializer(CreateDesc);
		Initializer.WriteArray(MakeConstArrayView(Indices));

		IndexBufferRHI = Initializer.Finalize();
	}
}

FArchive& operator<<(FArchive& Ar,FRawIndexBuffer& I)
{
	I.Indices.BulkSerialize( Ar );
	return Ar;
}

/*-----------------------------------------------------------------------------
	FRawIndexBuffer16or32
-----------------------------------------------------------------------------*/

/**
* Orders a triangle list for better vertex cache coherency.
*/
void FRawIndexBuffer16or32::CacheOptimize()
{
#if WITH_EDITOR
	CacheOptimizeIndexBuffer(Indices);
#endif
}

void FRawIndexBuffer16or32::ComputeIndexWidth()
{
	if (GetFeatureLevel() < ERHIFeatureLevel::SM5)
	{
		const int32 NumIndices = Indices.Num();
		bool bShouldUse32Bit = false;
		int32 i = 0;
		while (!bShouldUse32Bit && i < NumIndices)
		{
			bShouldUse32Bit = Indices[i] > MAX_uint16;
			i++;
		}
	
		b32Bit = bShouldUse32Bit;
	}
	else
	{
		b32Bit = true;
	}
}

void FRawIndexBuffer16or32::InitRHI(FRHICommandListBase& RHICmdList)
{
	const int32 IndexStride = b32Bit ? sizeof(uint32) : sizeof(uint16);
	const int32 NumIndices = Indices.Num();
	const uint32 Size = NumIndices * IndexStride;
	
	if (Size > 0)
	{
		// Create the index buffer.
		const FRHIBufferCreateDesc CreateDesc =
			FRHIBufferCreateDesc::CreateIndex(TEXT("FRawIndexBuffer"), Size, IndexStride)
			.AddUsage(EBufferUsageFlags::Static)
			.SetInitialState(ERHIAccess::VertexOrIndexBuffer | ERHIAccess::SRVMask)
			.SetInitActionInitializer();

		// Initialize the buffer.		
		if (b32Bit)
		{
			TRHIBufferInitializer<uint32> Initializer = RHICmdList.CreateBufferInitializer(CreateDesc);
			Initializer.WriteArray(Indices);
			IndexBufferRHI = Initializer.Finalize();
		}
		else
		{
			TRHIBufferInitializer<uint16> Initializer = RHICmdList.CreateBufferInitializer(CreateDesc);

			for (int32 Index = 0; Index < NumIndices; ++Index)
			{
				Initializer[Index] = Indices[Index];
			}

			IndexBufferRHI = Initializer.Finalize();
		}
	}

	// Undo/redo can destroy and recreate the render resources for UModels without rebuilding the
	// buffers, so the indices need to be saved when in the editor.
	if (!GIsEditor && !IsRunningCommandlet())
	{
		Indices.Empty();
	}
}

FArchive& operator<<(FArchive& Ar,FRawIndexBuffer16or32& I)
{
	I.Indices.BulkSerialize( Ar );
	return Ar;
}

/*-----------------------------------------------------------------------------
	FRawStaticIndexBuffer
-----------------------------------------------------------------------------*/

FRawStaticIndexBuffer::FRawStaticIndexBuffer(bool InNeedsCPUAccess)
	: IndexStorage(InNeedsCPUAccess)
	, CachedNumIndices(-1)
	, b32Bit(false)
{
}

FRawStaticIndexBuffer::~FRawStaticIndexBuffer() = default;

void FRawStaticIndexBuffer::SetIndices(const TArray<uint32>& InIndices, EIndexBufferStride::Type DesiredStride)
{
	int32 NumIndices = InIndices.Num();
	bool bShouldUse32Bit = false;

	// Figure out if we should store the indices as 16 or 32 bit.
	if (DesiredStride == EIndexBufferStride::Force32Bit)
	{
		bShouldUse32Bit = true;
	}
	else if (DesiredStride == EIndexBufferStride::AutoDetect)
	{
		int32 i = 0;
		while (!bShouldUse32Bit && i < NumIndices)
		{
			bShouldUse32Bit = InIndices[i] > MAX_uint16;
			i++;
		}
	}

	// Allocate storage for the indices.
	int32 IndexStride = bShouldUse32Bit ? sizeof(uint32) : sizeof(uint16);
	IndexStorage.Empty(IndexStride * NumIndices);
	IndexStorage.AddUninitialized(IndexStride * NumIndices);
	CachedNumIndices = NumIndices;

	// Store them!
	if (bShouldUse32Bit)
	{
		// If the indices are 32 bit we can just do a memcpy.
		check(IndexStorage.Num() == InIndices.Num() * InIndices.GetTypeSize());
		FMemory::Memcpy(IndexStorage.GetData(),InIndices.GetData(),IndexStorage.Num());
		b32Bit = true;
	}
	else
	{
		// Copy element by element demoting 32-bit integers to 16-bit.
		check(IndexStorage.Num() == InIndices.Num() * sizeof(uint16));
		uint16* DestIndices16Bit = (uint16*)IndexStorage.GetData();
		for (int32 i = 0; i < NumIndices; ++i)
		{
			DestIndices16Bit[i] = InIndices[i];
		}
		b32Bit = false;
	}
}

void FRawStaticIndexBuffer::InsertIndices( const uint32 At, const uint32* IndicesToAppend, const uint32 NumIndicesToAppend )
{
	if( NumIndicesToAppend > 0 )
	{
		const uint32 IndexStride = b32Bit ? sizeof( uint32 ) : sizeof( uint16 );

		IndexStorage.InsertUninitialized( At * IndexStride, NumIndicesToAppend * IndexStride );
		CachedNumIndices = IndexStorage.Num() / static_cast<int32>(IndexStride);
		uint8* const DestIndices = &IndexStorage[ At * IndexStride ];

		if( IndicesToAppend )
		{
			if( b32Bit )
			{
				// If the indices are 32 bit we can just do a memcpy.
				FMemory::Memcpy( DestIndices, IndicesToAppend, NumIndicesToAppend * IndexStride );
			}
			else
			{
				// Copy element by element demoting 32-bit integers to 16-bit.
				uint16* DestIndices16Bit = (uint16*)DestIndices;
				for( uint32 Index = 0; Index < NumIndicesToAppend; ++Index )
				{
					DestIndices16Bit[ Index ] = IndicesToAppend[ Index ];
				}
			}
		}
		else
		{
			// If no indices to insert were supplied, just clear the buffer
			FMemory::Memset( DestIndices, 0, NumIndicesToAppend * IndexStride );
		}
	}
}

void FRawStaticIndexBuffer::AppendIndices( const uint32* IndicesToAppend, const uint32 NumIndicesToAppend )
{
	InsertIndices( b32Bit ? IndexStorage.Num() / 4 : IndexStorage.Num() / 2, IndicesToAppend, NumIndicesToAppend );
}

void FRawStaticIndexBuffer::RemoveIndicesAt( const uint32 At, const uint32 NumIndicesToRemove )
{
	if( NumIndicesToRemove > 0 )
	{
		const int32 IndexStride = b32Bit ? sizeof( uint32 ) : sizeof( uint16 );
		IndexStorage.RemoveAt( At * IndexStride, NumIndicesToRemove * IndexStride );
		CachedNumIndices = IndexStorage.Num() / IndexStride;
	}
}

void FRawStaticIndexBuffer::GetCopy(TArray<uint32>& OutIndices) const
{
	int32 NumIndices = b32Bit ? (IndexStorage.Num() / 4) : (IndexStorage.Num() / 2);
	OutIndices.Empty(NumIndices);
	OutIndices.AddUninitialized(NumIndices);

	if (b32Bit)
	{
		// If the indices are 32 bit we can just do a memcpy.
		check(IndexStorage.Num() == OutIndices.Num() * OutIndices.GetTypeSize());
		FMemory::Memcpy(OutIndices.GetData(),IndexStorage.GetData(),IndexStorage.Num());
	}
	else
	{
		// Copy element by element promoting 16-bit integers to 32-bit.
		check(IndexStorage.Num() == OutIndices.Num() * sizeof(uint16));
		const uint16* SrcIndices16Bit = (const uint16*)IndexStorage.GetData();
		for (int32 i = 0; i < NumIndices; ++i)
		{
			OutIndices[i] = SrcIndices16Bit[i];
		}
	}
}

void FRawStaticIndexBuffer::ExpandTo32Bit()
{
	if (b32Bit)
		return;

	b32Bit = true;
	bool bAllowCpuAccess = IndexStorage.GetAllowCPUAccess();
	TResourceArray<uint8, INDEXBUFFER_ALIGNMENT> CopyIndex(bAllowCpuAccess);
	CopyIndex.Empty(sizeof(uint32) * CachedNumIndices);
	CopyIndex.AddUninitialized(sizeof(uint32) * CachedNumIndices);

	uint16* SrcIndices16Bit = (uint16*)IndexStorage.GetData();
	uint32* DstIndices32Bit = (uint32*)CopyIndex.GetData();
	for (int32 i = 0; i < CachedNumIndices; ++i)
	{
		DstIndices32Bit[i] = SrcIndices16Bit[i];
	}

	IndexStorage.Empty();
	IndexStorage = MoveTemp(CopyIndex);
}

const uint16* FRawStaticIndexBuffer::AccessStream16() const
{
	if (!b32Bit)
	{
		return reinterpret_cast<const uint16*>(IndexStorage.GetData());
	}
	return nullptr;
}

const uint32* FRawStaticIndexBuffer::AccessStream32() const
{
	if (b32Bit)
	{
		return reinterpret_cast<const uint32*>(IndexStorage.GetData());
	}
	return nullptr;
}

FIndexArrayView FRawStaticIndexBuffer::GetArrayView() const
{
	int32 NumIndices = b32Bit ? (IndexStorage.Num() / 4) : (IndexStorage.Num() / 2);
	return FIndexArrayView(IndexStorage.GetData(),NumIndices,b32Bit);
}

FBufferRHIRef FRawStaticIndexBuffer::CreateRHIBuffer(FRHICommandListBase& RHICmdList)
{
	const uint32 IndexStride = b32Bit ? sizeof(uint32) : sizeof(uint16);
	const uint32 SizeInBytes = IndexStorage.Num();

	if (GetNumIndices() > 0)
	{
		// Systems that generate data for GPUSkinPassThrough use index buffer as SRV, depending on platform, SRV may be created by default or by individual systems
		bool bIsShaderResource = AreIndexBuffersShaderResources(GMaxRHIShaderPlatform);

		// When bAllowCPUAccess is true, the meshes is likely going to be used for Niagara to spawn particles on mesh surface.
		// And it can be the case for CPU *and* GPU access: no differenciation today. That is why we create a SRV in this case.
		// This also avoid setting lots of states on all the members of all the different buffers used by meshes. Follow up: https://jira.it.epicgames.net/browse/UE-69376.
		bIsShaderResource |= IndexStorage.GetAllowCPUAccess();

		const EBufferUsageFlags BufferFlags = EBufferUsageFlags::Static | (bIsShaderResource ? EBufferUsageFlags::ShaderResource : EBufferUsageFlags::None);

		const static FLazyName ClassName32(TEXT("FRawStaticIndexBuffer32"));
		const static FLazyName ClassName16(TEXT("FRawStaticIndexBuffer16"));

		const TCHAR* BufferName = Is32Bit() ? TEXT("FRawStaticIndexBuffer32") : TEXT("FRawStaticIndexBuffer16");

		// Create the index buffer.
		FRHIBufferCreateDesc Desc =
			SizeInBytes
			? (FRHIBufferCreateDesc::CreateIndex(BufferName, SizeInBytes, IndexStride).AddUsage(BufferFlags))
			: FRHIBufferCreateDesc::CreateNull(BufferName);

		Desc.SetClassName(Is32Bit() ? ClassName32 : ClassName16);
		Desc.SetOwnerName(GetOwnerName());
		Desc.SetInitialState(ERHIAccess::VertexOrIndexBuffer | ERHIAccess::SRVMask);
		Desc.SetInitActionResourceArray(&IndexStorage);

		return RHICmdList.CreateBuffer(Desc);
	}
	return nullptr;
}

void FRawStaticIndexBuffer::InitRHIForStreaming(FRHIBuffer* IntermediateBuffer, FRHIResourceReplaceBatcher& Batcher)
{
	if (IndexBufferRHI && IntermediateBuffer)
	{
		Batcher.EnqueueReplace(IndexBufferRHI, IntermediateBuffer);
	}
}

void FRawStaticIndexBuffer::ReleaseRHIForStreaming(FRHIResourceReplaceBatcher& Batcher)
{
	if (IndexBufferRHI)
	{
		Batcher.EnqueueReplace(IndexBufferRHI, nullptr);
	}
}

void FRawStaticIndexBuffer::InitRHI(FRHICommandListBase& RHICmdList)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FRawStaticIndexBuffer::InitRHI);
	IndexBufferRHI = CreateRHIBuffer(RHICmdList);
}

void FRawStaticIndexBuffer::Serialize(FArchive& Ar, bool bNeedsCPUAccess)
{
	IndexStorage.SetAllowCPUAccess(bNeedsCPUAccess);

	if (Ar.UEVer() < VER_UE4_SUPPORT_32BIT_STATIC_MESH_INDICES)
	{
		TResourceArray<uint16,INDEXBUFFER_ALIGNMENT> LegacyIndices;

		b32Bit = false;
		LegacyIndices.BulkSerialize(Ar);
		int32 NumIndices = LegacyIndices.Num();
		int32 IndexStride = sizeof(uint16);
		IndexStorage.Empty(NumIndices * IndexStride);
		IndexStorage.AddUninitialized(NumIndices * IndexStride);
		FMemory::Memcpy(IndexStorage.GetData(),LegacyIndices.GetData(),IndexStorage.Num());
		CachedNumIndices = IndexStorage.Num() / (b32Bit ? 4 : 2);
	}
	else
	{
		Ar << b32Bit;
		IndexStorage.BulkSerialize(Ar);
		CachedNumIndices = IndexStorage.Num() / (b32Bit ? 4 : 2);

		// Set when cooking for Android if the 16-bit index data potentially needs to be converted to 32-bit on load to work around bugs on certain devices.
		bool bShouldExpandTo32Bit = false;

		if (Ar.IsCooking() && CachedNumIndices > 0 && !b32Bit)
		{
			bShouldExpandTo32Bit = Ar.CookingTarget()->ShouldExpandTo32Bit((const uint16*)IndexStorage.GetData(), CachedNumIndices);
		}

		Ar << bShouldExpandTo32Bit;

		if (Ar.IsLoading() && bShouldExpandTo32Bit && FPlatformMisc::Expand16BitIndicesTo32BitOnLoad())
		{
			ExpandTo32Bit();
		}
	}
}

void FRawStaticIndexBuffer::SerializeMetaData(FArchive& Ar)
{
	Ar << CachedNumIndices << b32Bit;
}

void FRawStaticIndexBuffer::ClearMetaData()
{
	CachedNumIndices = -1;
}

void FRawStaticIndexBuffer::Discard()
{
    IndexStorage.SetAllowCPUAccess(false);
    IndexStorage.Discard();
}

bool FRawStaticIndexBuffer16or32Interface::IsSRVCreatedByDefault(bool bAllowCPUAccess) const
{
	// Use platform specific features like ManualVertexFetch and GPUSkinCache to infer platform capability
	bool bSRV = AreBufferSRVsAlwaysCreatedByDefault(GMaxRHIShaderPlatform);
	// When bAllowCPUAccess is true, the meshes is likely going to be used for Niagara to spawn particles on mesh surface.
	// And it can be the case for CPU *and* GPU access: no differenciation today. That is why we create a SRV in this case.
	// This also avoid setting lots of states on all the members of all the different buffers used by meshes. Follow up: https://jira.it.epicgames.net/browse/UE-69376.
	bSRV |= bAllowCPUAccess;
	return bSRV;
}

bool FRawStaticIndexBuffer16or32Interface::IsShaderResource(bool bAllowCPUAccess) const
{
	// In the case that AreBufferSRVsAlwaysCreatedByDefault is false, there might be other systems (eg. MeshDeformer)
	// that still want to create SRVs on demand only for specific meshes in the project.
	bool bCanBeShaderResource = AreIndexBuffersShaderResources(GMaxRHIShaderPlatform);
	// When bAllowCPUAccess is true, the meshes is likely going to be used for Niagara to spawn particles on mesh surface.
	// And it can be the case for CPU *and* GPU access: no differenciation today. That is why we create a SRV in this case.
	// This also avoid setting lots of states on all the members of all the different buffers used by meshes. Follow up: https://jira.it.epicgames.net/browse/UE-69376.
	bCanBeShaderResource |= bAllowCPUAccess;
	return bCanBeShaderResource;	
}

// Deprecated
bool FRawStaticIndexBuffer16or32Interface::IsSRVNeeded(bool bAllowCPUAccess) const
{
	// Systems that generate data for GPUSkinPassThrough use index buffer as SRV.
	bool bSRV = RHISupportsManualVertexFetch(GMaxRHIShaderPlatform) || IsGPUSkinPassThroughSupported(GMaxRHIShaderPlatform);
	// When bAllowCPUAccess is true, the meshes is likely going to be used for Niagara to spawn particles on mesh surface.
	// And it can be the case for CPU *and* GPU access: no differenciation today. That is why we create a SRV in this case.
	// This also avoid setting lots of states on all the members of all the different buffers used by meshes. Follow up: https://jira.it.epicgames.net/browse/UE-69376.
	bSRV |= bAllowCPUAccess;
	return bSRV;
}

void FRawStaticIndexBuffer16or32Interface::InitRHIForStreaming(FRHIBuffer* IntermediateBuffer, size_t IndexSize, FRHIResourceReplaceBatcher& Batcher)
{
	if (IndexBufferRHI && IntermediateBuffer)
	{
		Batcher.EnqueueReplace(IndexBufferRHI, IntermediateBuffer);
	}
}

void FRawStaticIndexBuffer16or32Interface::ReleaseRHIForStreaming(FRHIResourceReplaceBatcher& Batcher)
{
	if (IndexBufferRHI)
	{
		Batcher.EnqueueReplace(IndexBufferRHI, nullptr);
	}
}

FBufferRHIRef FRawStaticIndexBuffer16or32Interface::CreateRHIIndexBufferInternal(
	FRHICommandListBase& RHICmdList,
	const TCHAR* InDebugName,
	const FName& InOwnerName,
	int32 IndexCount,
	size_t IndexSize,
	FResourceArrayInterface* ResourceArray,
	bool bIsShaderResource
)
{
	const uint32 Size = IndexCount * IndexSize;

	FRHIBufferCreateDesc CreateDesc = Size
		? (FRHIBufferCreateDesc::CreateIndex(InDebugName, Size, IndexSize).AddUsage(EBufferUsageFlags::Static))
		: FRHIBufferCreateDesc::CreateNull(InDebugName);

	CreateDesc.SetClassName(InDebugName)
		.SetOwnerName(InOwnerName)
		.SetInitialState(ERHIAccess::VertexOrIndexBuffer | ERHIAccess::SRVMask);

	if (!CreateDesc.IsNull())
	{
		if (bIsShaderResource)
		{
			// EBufferUsageFlags::ShaderResource is needed for SkinCache RecomputeSkinTangents
			CreateDesc.AddUsage(EBufferUsageFlags::ShaderResource);
		}

		if (ResourceArray)
		{
			CreateDesc.SetInitActionResourceArray(ResourceArray);
		}
	}

	return RHICmdList.CreateBuffer(CreateDesc);
}

/*-----------------------------------------------------------------------------
FRawStaticIndexBuffer16or32
-----------------------------------------------------------------------------*/

/**
* Orders a triangle list for better vertex cache coherency.
*/
template <typename INDEX_TYPE>
void FRawStaticIndexBuffer16or32<INDEX_TYPE>::CacheOptimize()
{
#if WITH_EDITOR
	CacheOptimizeIndexBuffer(Indices);
	CachedNumIndices = Indices.Num();
#endif
}

