// Copyright Epic Games, Inc. All Rights Reserved.

#include "RayTracingVisualizationData.h"

#include "RaytracingDebugDefinitions.h"
#include "RenderUtils.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "RHIGlobals.h"

#define LOCTEXT_NAMESPACE "FRayTracingVisualizationData"

static FName GRayTracingVisualizeMode = NAME_None;
static FAutoConsoleVariableRef CVarRayTracingVisualize(
	TEXT("r.RayTracing.Visualize"),
	GRayTracingVisualizeMode,
	TEXT("Sets the ray tracing debug visualization mode (default = None - Driven by viewport menu) .\n"),
	ECVF_RenderThreadSafe
);

static FAutoConsoleVariableDeprecated CVarRayTracingDebugMode_Deprecated(TEXT("r.RayTracing.DebugVisualizationMode"), TEXT("r.RayTracing.Visualize"), TEXT("5.6"));

FRayTracingVisualizationData::FRayTracingVisualizationData()
{
	// always supported (as long as either inline RT or RT shaders work)
	AddVisualizationMode(TEXT("Barycentrics"), LOCTEXT("Barycentrics", "Barycentrics"), FModeType::Standard, RAY_TRACING_DEBUG_VIZ_BARYCENTRICS, false);

	{
		// runs basic lighting calculations on hits
		AddVisualizationMode(TEXT("PrimaryRays"), LOCTEXT("PrimaryRays", "Primary Rays"), FModeType::Standard, RAY_TRACING_DEBUG_VIZ_PRIMARY_RAYS, true);

		// all of these fields reflect entries in the payload which require running a CHS
		AddVisualizationMode(TEXT("Radiance"), LOCTEXT("Radiance", "Radiance"), FModeType::Other, RAY_TRACING_DEBUG_VIZ_RADIANCE, true);
		AddVisualizationMode(TEXT("WorldNormal"), LOCTEXT("WorldNormal", "World Normal"), FModeType::Standard, RAY_TRACING_DEBUG_VIZ_WORLD_NORMAL, false);
		AddVisualizationMode(TEXT("BaseColor"), LOCTEXT("BaseColor", "Base Color"), FModeType::Standard, RAY_TRACING_DEBUG_VIZ_BASE_COLOR, false);
		AddVisualizationMode(TEXT("DiffuseColor"), LOCTEXT("DiffuseColor", "Diffuse Color"), FModeType::Other, RAY_TRACING_DEBUG_VIZ_DIFFUSE_COLOR, false);
		AddVisualizationMode(TEXT("SpecularColor"), LOCTEXT("SpecularColor", "Specular Color"), FModeType::Other, RAY_TRACING_DEBUG_VIZ_SPECULAR_COLOR, false);
		AddVisualizationMode(TEXT("Opacity"), LOCTEXT("Opacity", "Opacity"), FModeType::Other, RAY_TRACING_DEBUG_VIZ_OPACITY, false);
		AddVisualizationMode(TEXT("Metallic"), LOCTEXT("Metallic", "Metallic"), FModeType::Other, RAY_TRACING_DEBUG_VIZ_METALLIC, false);
		AddVisualizationMode(TEXT("Specular"), LOCTEXT("Specular", "Specular"), FModeType::Other, RAY_TRACING_DEBUG_VIZ_SPECULAR, false);
		AddVisualizationMode(TEXT("Roughness"), LOCTEXT("Roughness", "Roughness"), FModeType::Other, RAY_TRACING_DEBUG_VIZ_ROUGHNESS, false);
		AddVisualizationMode(TEXT("Ior"), LOCTEXT("Ior", "Ior"), FModeType::Other, RAY_TRACING_DEBUG_VIZ_IOR, false);
		AddVisualizationMode(TEXT("ShadingModelID"), LOCTEXT("ShadingModelID", "Shading Model ID"), FModeType::Other, RAY_TRACING_DEBUG_VIZ_SHADING_MODEL, false);
		AddVisualizationMode(TEXT("BlendingMode"), LOCTEXT("BlendingMode", "Blending Mode"), FModeType::Other, RAY_TRACING_DEBUG_VIZ_BLENDING_MODE, false);
		AddVisualizationMode(TEXT("PrimitiveLightingChannelMask"), LOCTEXT("PrimitiveLightingChannelMask", "Primitive Lighting Channel Mask"), FModeType::Other, RAY_TRACING_DEBUG_VIZ_LIGHTING_CHANNEL_MASK, false);
		AddVisualizationMode(TEXT("CustomData"), LOCTEXT("CustomData", "Custom Data"), FModeType::Other, RAY_TRACING_DEBUG_VIZ_CUSTOM_DATA, false);
		AddVisualizationMode(TEXT("GBufferAO"), LOCTEXT("GBufferAO", "GBuffer AO"), FModeType::Other, RAY_TRACING_DEBUG_VIZ_GBUFFER_AO, false);
		AddVisualizationMode(TEXT("IndirectIrradiance"), LOCTEXT("IndirectIrradiance", "Indirect Irradiance"), FModeType::Other, RAY_TRACING_DEBUG_VIZ_INDIRECT_IRRADIANCE, true);
		AddVisualizationMode(TEXT("WorldPosition"), LOCTEXT("WorldPosition", "World Position"), FModeType::Other, RAY_TRACING_DEBUG_VIZ_WORLD_POSITION, false);
		AddVisualizationMode(TEXT("HitKind"), LOCTEXT("HitKind", "Hit Kind"), FModeType::Other, RAY_TRACING_DEBUG_VIZ_HITKIND, false);
		AddVisualizationMode(TEXT("WorldTangent"), LOCTEXT("WorldTangent", "World Tangent"), FModeType::Other, RAY_TRACING_DEBUG_VIZ_WORLD_TANGENT, false);
		AddVisualizationMode(TEXT("Anisotropy"), LOCTEXT("Anisotropy", "Anisotropy"), FModeType::Other, RAY_TRACING_DEBUG_VIZ_ANISOTROPY, false);

		// debugging the geometry itself
		AddVisualizationMode(TEXT("Instances"), LOCTEXT("Instances", "Instances"), FModeType::Standard, RAY_TRACING_DEBUG_VIZ_INSTANCES, false);
		AddVisualizationMode(TEXT("Triangles"), LOCTEXT("Triangles", "Triangles"), FModeType::Standard, RAY_TRACING_DEBUG_VIZ_TRIANGLES, false);
		if (GRHIGlobals.RayTracing.SupportsClusterOps)
		{
			AddVisualizationMode(TEXT("Clusters"), LOCTEXT("Clusters", "Clusters"), FModeType::Standard, RAY_TRACING_DEBUG_VIZ_CLUSTERS, false);
		}
		AddVisualizationMode(TEXT("DynamicInstances"), LOCTEXT("DynamicInstances", "Dynamic Instances"), FModeType::Standard, RAY_TRACING_DEBUG_VIZ_DYNAMIC_INSTANCES, false);
		AddVisualizationMode(TEXT("ProxyType"), LOCTEXT("ProxyType", "Proxy Type"), FModeType::Standard, RAY_TRACING_DEBUG_VIZ_PROXY_TYPE, false);
		AddVisualizationMode(TEXT("Picker"), LOCTEXT("Picker", "Picker"), FModeType::Standard, RAY_TRACING_DEBUG_VIZ_PICKER, false);
		AddVisualizationMode(TEXT("LightGridOccupancy"), LOCTEXT("LightGridOccupancy", "Light Grid Occupancy"), FModeType::Standard, RAY_TRACING_DEBUG_VIZ_LIGHT_GRID_COUNT, false);

		// performance
		AddVisualizationMode(TEXT("InstanceOverlap"), LOCTEXT("InstanceOverlap", "Instance Overlap"), FModeType::Performance, RAY_TRACING_DEBUG_VIZ_INSTANCE_OVERLAP, false);
		AddVisualizationMode(TEXT("TriangleHitCount"), LOCTEXT("TriangleHitCount", "Triangle Hit Count"), FModeType::Performance, RAY_TRACING_DEBUG_VIZ_TRIANGLE_HITCOUNT, false);
		AddVisualizationMode(TEXT("HitCountPerInstance"), LOCTEXT("HitCountPerInstance", "Hit Count Per Instance"), FModeType::Performance, RAY_TRACING_DEBUG_VIZ_HITCOUNT_PER_INSTANCE, false);

		AddVisualizationMode(TEXT("Traversal Primary Node"), LOCTEXT("TraversalPrimaryNode", "Traversal Primary Node"), FModeType::Performance, RAY_TRACING_DEBUG_VIZ_TRAVERSAL_PRIMARY_NODE, false, /*bHiddenInEditor*/ true);
		AddVisualizationMode(TEXT("Traversal Primary Cluster"), LOCTEXT("TraversalPrimaryCluster", "Traversal Primary Cluster"), FModeType::Performance, RAY_TRACING_DEBUG_VIZ_TRAVERSAL_PRIMARY_CLUSTER, false, /*bHiddenInEditor*/ true);
		AddVisualizationMode(TEXT("Traversal Primary Triangle"), LOCTEXT("TraversalPrimaryTriangle", "Traversal Primary Triangle"), FModeType::Performance, RAY_TRACING_DEBUG_VIZ_TRAVERSAL_PRIMARY_TRIANGLE, false, /*bHiddenInEditor*/ true);
		AddVisualizationMode(TEXT("Traversal Primary All"), LOCTEXT("TraversalPrimaryAll", "Traversal Primary All"), FModeType::Performance, RAY_TRACING_DEBUG_VIZ_TRAVERSAL_PRIMARY_ALL, false, /*bHiddenInEditor*/ true);
		AddVisualizationMode(TEXT("Traversal Primary Statistics"), LOCTEXT("TraversalPrimaryStatistics", "Traversal Primary Statistics"), FModeType::Performance, RAY_TRACING_DEBUG_VIZ_TRAVERSAL_PRIMARY_STATISTICS, false, /*bHiddenInEditor*/ true);

		AddVisualizationMode(TEXT("Traversal Secondary Node"), LOCTEXT("TraversalSecondaryNode", "Traversal Secondary Node"), FModeType::Performance, RAY_TRACING_DEBUG_VIZ_TRAVERSAL_SECONDARY_NODE, false, /*bHiddenInEditor*/ true);
		AddVisualizationMode(TEXT("Traversal Secondary Cluster"), LOCTEXT("TraversalSecondaryCluster", "Traversal Secondary Cluster"), FModeType::Performance, RAY_TRACING_DEBUG_VIZ_TRAVERSAL_SECONDARY_CLUSTER, false, /*bHiddenInEditor*/ true);
		AddVisualizationMode(TEXT("Traversal Secondary Triangle"), LOCTEXT("TraversalSecondaryTriangle", "Traversal Secondary Triangle"), FModeType::Performance, RAY_TRACING_DEBUG_VIZ_TRAVERSAL_SECONDARY_TRIANGLE, false, /*bHiddenInEditor*/ true);
		AddVisualizationMode(TEXT("Traversal Secondary All"), LOCTEXT("TraversalSecondaryAll", "Traversal Secondary All"), FModeType::Performance, RAY_TRACING_DEBUG_VIZ_TRAVERSAL_SECONDARY_ALL, false, /*bHiddenInEditor*/ true);
		AddVisualizationMode(TEXT("Traversal Secondary Statistics"), LOCTEXT("TraversalSecondaryStatistics", "Traversal Secondary Statistics"), FModeType::Performance, RAY_TRACING_DEBUG_VIZ_TRAVERSAL_SECONDARY_STATISTICS, false, /*bHiddenInEditor*/ true);

		if (GRHISupportsShaderTimestamp)
		{
			AddVisualizationMode(TEXT("Timing Traversal"), LOCTEXT("TimingTraversal", "Timing - Traversal"), FModeType::Timing, RAY_TRACING_DEBUG_VIZ_TIMING_TRAVERSAL, false);
			AddVisualizationMode(TEXT("Timing Material"), LOCTEXT("TimingMaterial", "Timing - Material"), FModeType::Timing, RAY_TRACING_DEBUG_VIZ_TIMING_MATERIAL, false);
			AddVisualizationMode(TEXT("Timing AHS"), LOCTEXT("TimingAHS", "Timing - Material (Alpha Only)"), FModeType::Timing, RAY_TRACING_DEBUG_VIZ_TIMING_ANY_HIT, false);
		}

		if (Substrate::IsSubstrateEnabled())
		{
			AddVisualizationMode(TEXT("SubstrateMaterialProperties"), LOCTEXT("SubstrateMaterialProperties", "Substrate Material Properties"), FModeType::Other, RAY_TRACING_DEBUG_VIZ_SUBSTRATE_DATA, true);
		}
	}

	ConfigureConsoleCommand();
}

void FRayTracingVisualizationData::AddVisualizationMode(
	const TCHAR* ModeString,
	const FText& ModeText,
	const FModeType ModeType,
	int32 ModeID,
	bool bTonemapped,
	bool bHiddenInEditor
)
{
	if (!GRHIGlobals.RayTracing.SupportsShaders && !(GRHIGlobals.RayTracing.SupportsInlineRayTracing && RayTracingDebugModeSupportsInline(ModeID)))
	{
		return;
	}

	const FName ModeName = FName(ModeString);

	FModeRecord& Record	= ModeMap.Emplace(ModeName);
	Record.ModeString			= FString(ModeString);
	Record.ModeName				= ModeName;
	Record.ModeText				= ModeText;
	Record.ModeDesc				= FText::GetEmpty();
	Record.ModeType				= ModeType;
	Record.ModeID				= ModeID;
	Record.bTonemapped			= bTonemapped;
	Record.bHiddenInEditor		= bHiddenInEditor;
}

FText FRayTracingVisualizationData::GetModeDisplayName(const FName& InModeName) const
{
	if (const FModeRecord* Record = ModeMap.Find(InModeName))
	{
		return Record->ModeText;
	}
	else
	{
		return FText::GetEmpty();
	}
}

int32 FRayTracingVisualizationData::GetModeID(const FName& InModeName) const
{
	if (const FModeRecord* Record = ModeMap.Find(InModeName))
	{
		return Record->ModeID;
	}
	else
	{
		return INDEX_NONE;
	}
}

bool FRayTracingVisualizationData::GetModeTonemapped(const FName& InModeName) const
{
	if (const FModeRecord* Record = ModeMap.Find(InModeName))
	{
		return Record->bTonemapped;
	}
	else
	{
		return false;
	}
}

bool FRayTracingVisualizationData::HasOverrides() const
{
	return GRayTracingVisualizeMode != NAME_None && GetModeID(GRayTracingVisualizeMode) != INDEX_NONE;
}

FName FRayTracingVisualizationData::ApplyOverrides(const FName& InModeName) const
{
	check(IsInParallelRenderingThread());
	return HasOverrides() ? GRayTracingVisualizeMode : InModeName;
}

void FRayTracingVisualizationData::ConfigureConsoleCommand()
{
	FString AvailableVisualizationModes;
	for (TModeMap::TConstIterator It = ModeMap.CreateConstIterator(); It; ++It)
	{
		const FModeRecord& Record = It.Value();
		AvailableVisualizationModes += FString("\n  ");
		AvailableVisualizationModes += Record.ModeString;
	}

	ConsoleDocumentationVisualizationMode = TEXT("When the viewport view-mode is set to 'Ray Tracing Debug', this command specifies which of the various modes to display. Values entered other than the allowed values shown below will be ignored.");
	ConsoleDocumentationVisualizationMode += AvailableVisualizationModes;

	CVarRayTracingVisualize->SetHelp(*ConsoleDocumentationVisualizationMode);
}

FRayTracingVisualizationData& GetRayTracingVisualizationData()
{
	static FRayTracingVisualizationData GRayTracingVisualizationData;
	return GRayTracingVisualizationData;
}

#undef LOCTEXT_NAMESPACE
