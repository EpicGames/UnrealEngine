// Copyright Epic Games, Inc. All Rights Reserved.

#include "Engine/RectLight.h"
#include "Components/RectLightComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RectLight)

ARectLight::ARectLight(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer.SetDefaultSubobjectClass<URectLightComponent>(TEXT("LightComponent0")))
{
	RectLightComponent = CastChecked<URectLightComponent>(GetLightComponent());
	RectLightComponent->Mobility = EComponentMobility::Stationary;

	RootComponent = RectLightComponent;
}

#if WITH_EDITOR
void ARectLight::EditorApplyScale(const FVector& DeltaScale, const FVector* PivotLocation, bool bAltDown, bool bShiftDown, bool bCtrlDown)
{
	const FVector ModifiedScale = DeltaScale * ( AActor::bUsePercentageBasedScaling ? 10000.0f : 100.0f );

	FMath::ApplyScaleToFloat( RectLightComponent->AttenuationRadius, ModifiedScale, 1.0f );

	PostEditChange();
}
#endif


