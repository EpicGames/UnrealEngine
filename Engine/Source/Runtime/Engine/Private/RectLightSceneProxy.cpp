// Copyright Epic Games, Inc. All Rights Reserved.

#include "RectLightSceneProxy.h"
#include "RectLightSceneProxyDesc.h"
#include "Components/RectLightComponent.h"
#include "SceneManagement.h"
#include "SceneInterface.h"
#include "SceneView.h"

extern int32 GAllowPointLightCubemapShadows;

FRectLightSceneProxy::FRectLightSceneProxy(const URectLightComponent* Component)
	: FRectLightSceneProxy(URectLightComponent::BuildSceneProxyDesc(*Component))
{
}

FRectLightSceneProxy::FRectLightSceneProxy(const FRectLightSceneProxyDesc& LightDesc)
	: FLocalLightSceneProxy(LightDesc)
	, SourceWidth(FMath::Max(1, LightDesc.SourceWidth))
	, SourceHeight(FMath::Max(1, LightDesc.SourceHeight))
	, BarnDoorAngle(FMath::Clamp(LightDesc.BarnDoorAngle, 0.f, GetRectLightBarnDoorMaxAngle()))
	, BarnDoorLength(FMath::Max(0.1f, LightDesc.BarnDoorLength))
	, SourceTexture(LightDesc.SourceTexture)
	, RectAtlasId(~0u)
	, LightFunctionConeAngleTangent(LightDesc.LightFunctionConeAngle > 0 ? FMath::Tan(FMath::Clamp(LightDesc.LightFunctionConeAngle, 0.0f, 89.0f) * (float)UE_PI / 180.0f) : 0.0f)
	, SourceTextureScaleOffset(FVector4f(
		FMath::Clamp(LightDesc.SourceTextureScale.X, 0.f, 1.f),
		FMath::Clamp(LightDesc.SourceTextureScale.Y, 0.f, 1.f),
		FMath::Clamp(LightDesc.SourceTextureOffset.X, 0.f, 1.f),
		FMath::Clamp(LightDesc.SourceTextureOffset.Y, 0.f, 1.f)))
{
}

FRectLightSceneProxy::~FRectLightSceneProxy() {}

bool FRectLightSceneProxy::IsRectLight() const
{
	return true;
}

bool FRectLightSceneProxy::HasSourceTexture() const
{
	return SourceTexture != nullptr;
}

/** Accesses parameters needed for rendering the light. */
void FRectLightSceneProxy::GetLightShaderParameters(FLightRenderParameters& LightParameters, uint32 Flags) const
{
	FLinearColor LightColor = GetColor();
	LightColor /= 0.5f * SourceWidth * SourceHeight;
	LightParameters.WorldPosition = GetOrigin();
	LightParameters.InvRadius = InvRadius;
	LightParameters.Color = LightColor;
	LightParameters.FalloffExponent = 0.0f;

	LightParameters.Direction = FVector3f(-GetDirection());
	LightParameters.Tangent = FVector3f(WorldToLight.M[0][2], WorldToLight.M[1][2], WorldToLight.M[2][2]);
	LightParameters.SpotAngles = FVector2f(-2.0f, 1.0f);
	LightParameters.SpecularScale = FMath::Clamp(SpecularScale, 0.f, 1.f);
	LightParameters.DiffuseScale = FMath::Clamp(DiffuseScale, 0.f, 1.f);
	LightParameters.SourceRadius = SourceWidth * 0.5f;
	LightParameters.SoftSourceRadius = 0.0f;
	LightParameters.SourceLength = SourceHeight * 0.5f;
	LightParameters.RectLightBarnCosAngle = FMath::Cos(FMath::DegreesToRadians(BarnDoorAngle));
	LightParameters.RectLightBarnLength = BarnDoorLength;
	LightParameters.RectLightAtlasUVOffset = FVector2f::ZeroVector;
	LightParameters.RectLightAtlasUVScale = FVector2f::ZeroVector;
	LightParameters.RectLightAtlasMaxLevel = FLightRenderParameters::GetRectLightAtlasInvalidMIPLevel();
	LightParameters.IESAtlasIndex = INDEX_NONE;
	LightParameters.InverseExposureBlend = InverseExposureBlend;
	LightParameters.LightFunctionAtlasLightIndex = GetLightFunctionAtlasLightIndex();
	LightParameters.bAffectsTranslucentLighting = AffectsTranslucentLighting() ? 1 : 0;

	if (IESAtlasId != ~0)
	{
		GetSceneInterface()->GetLightIESAtlasSlot(this, &LightParameters);
	}

	if (RectAtlasId != ~0u)
	{
		GetSceneInterface()->GetRectLightAtlasSlot(this, &LightParameters);
	}

	// Render RectLight approximately as SpotLight if the requester does not support rect light (e.g., translucent light grid or mobile)
	if (!!(Flags & ELightShaderParameterFlags::RectAsSpotLight))
	{
		float ClampedOuterConeAngle = FMath::DegreesToRadians(89.001f);
		float ClampedInnerConeAngle = FMath::DegreesToRadians(70.0f);
		float CosOuterCone = FMath::Cos(ClampedOuterConeAngle);
		float CosInnerCone = FMath::Cos(ClampedInnerConeAngle);
		float InvCosConeDifference = 1.0f / (CosInnerCone - CosOuterCone);

		LightParameters.Color = GetColor();
		LightParameters.FalloffExponent = 8.0f;
		LightParameters.SpotAngles = FVector2f(CosOuterCone, InvCosConeDifference);
		LightParameters.SourceRadius = (SourceWidth + SourceHeight) * 0.5 * 0.5f;
		LightParameters.SourceLength = 0.0f;
		LightParameters.RectLightBarnCosAngle = 0.0f;
		LightParameters.RectLightBarnLength = -2.0f;
	}
}

/**
* Sets up a projected shadow initializer for shadows from the entire scene.
* @return True if the whole-scene projected shadow should be used.
*/
bool FRectLightSceneProxy::GetWholeSceneProjectedShadowInitializer(const FSceneViewFamily& ViewFamily, TArray<FWholeSceneProjectedShadowInitializer, TInlineAllocator<6> >& OutInitializers) const
{
	if (ViewFamily.GetFeatureLevel() >= ERHIFeatureLevel::SM5
		&& GAllowPointLightCubemapShadows != 0)
	{
		FWholeSceneProjectedShadowInitializer& OutInitializer = *new(OutInitializers) FWholeSceneProjectedShadowInitializer;
		OutInitializer.PreShadowTranslation = -GetLightToWorld().GetOrigin();
		OutInitializer.WorldToLight = GetWorldToLight().RemoveTranslation();
		OutInitializer.Scales = FVector2D(1, 1);
		OutInitializer.SubjectBounds = FBoxSphereBounds(FVector(0, 0, 0), FVector(Radius, Radius, Radius), Radius);
		OutInitializer.WAxis = FVector4(0, 0, 1, 0);
		OutInitializer.MinLightW = 0.1f;
		OutInitializer.MaxDistanceToCastInLightW = Radius;
		OutInitializer.bOnePassPointLightShadow = true;

		OutInitializer.bRayTracedDistanceField = UseRayTracedDistanceFieldShadows() && DoesPlatformSupportDistanceFieldShadowing(ViewFamily.GetShaderPlatform());
		return true;
	}

	return false;
}