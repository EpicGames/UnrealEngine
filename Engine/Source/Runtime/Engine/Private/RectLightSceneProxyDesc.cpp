// Copyright Epic Games, Inc. All Rights Reserved.

#include "RectLightSceneProxyDesc.h"

void FRectLightSceneProxyDesc::Serialize(FArchive& Ar)
{
	FLocalLightSceneProxyDesc::Serialize(Ar);

	Ar << SourceWidth;
	Ar << SourceHeight;
	Ar << BarnDoorAngle;
	Ar << BarnDoorLength;
	Ar << LightFunctionConeAngle;
	Ar << SourceTexture;
	Ar << SourceTextureScale;
	Ar << SourceTextureOffset;
}
