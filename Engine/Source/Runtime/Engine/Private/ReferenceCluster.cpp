// Copyright Epic Games, Inc. All Rights Reserved.

#include "ReferenceCluster.h"
#include "Misc/Guid.h"

TArray<TArray<FGuid>> GenerateObjectsClusters(const TArray<TPair<FGuid, TArray<FGuid>>>& InObjects)
{
	TArray<TArray<FGuid>> Result;
	Result.Reserve(InObjects.Num());

	TMap<FGuid, TSet<FGuid>> Graph;
	Graph.Reserve(InObjects.Num());

	for (const auto& [Object, References] : InObjects)
	{
		TSet<FGuid>& ObjectReferences = Graph.FindOrAdd(Object);
		ObjectReferences.Reserve(References.Num());

		for (const FGuid& Reference : References)
		{
			ObjectReferences.Add(Reference);
			Graph.FindOrAdd(Reference).FindOrAdd(Object);
		}
	}

	TSet<FGuid> VisitedObjects;
	VisitedObjects.Reserve(InObjects.Num());

	TArray<FGuid> ObjectStack;
	for (const auto& ObjectNode : Graph)
	{
		if (!VisitedObjects.Contains(ObjectNode.Key))
		{
			TArray<FGuid>& NewCluster = Result.Emplace_GetRef();

			ObjectStack.Reset();
			ObjectStack.Add(ObjectNode.Key);

			while (!ObjectStack.IsEmpty())
			{
				const FGuid Object = ObjectStack.Pop(EAllowShrinking::No);
				if (!VisitedObjects.Contains(Object))
				{
					VisitedObjects.Add(Object);
					NewCluster.Add(Object);
					ObjectStack.Append(Graph[Object].Array());
				}
			}
		}
	}

	return MoveTemp(Result);
}