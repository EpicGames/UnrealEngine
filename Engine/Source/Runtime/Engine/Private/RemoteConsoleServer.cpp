// Copyright Epic Games, Inc. All Rights Reserved.

// See RemoteConsoleServer.h for protocol documentation.

#include "RemoteConsoleServer.h"

#if WITH_REMOTE_CONSOLE_SERVER

#include "Engine/Engine.h"
#include "Engine/Console.h"
#include "Engine/GameViewportClient.h"
#include "HAL/IConsoleManager.h"
#include "HAL/Event.h"
#include "HAL/PlatformProcess.h"
#include "HAL/PlatformTime.h"
#include "Serialization/JsonSerializer.h"
#include "Dom/JsonObject.h"

#include <atomic>

static constexpr uint32 GExecTimeoutMs = 10000;
// Trie walk timeout -- falls back to ForEachConsoleObjectThatStartsWith.
static constexpr uint32 GTrieWalkTimeoutMs = 5000;
static constexpr uint32 GReadLineTimeoutMs = 5000;
static constexpr float GPollIntervalSeconds = 0.01f;
static constexpr double GConnectionTimeoutSeconds = 300.0;

static int32 SafeEnd(int32 Offset, int32 Limit)
{
	return (Offset <= MAX_int32 - Limit) ? Offset + Limit : MAX_int32;
}

// Append TCHAR as UTF-8 directly into TArray<char>, avoiding FTCHARToUTF8 heap allocation.
// Safe: with explicit SourceLen, ConvertedLength returns byte count WITHOUT null terminator,
// and Convert writes exactly that many bytes without appending a null terminator.
// (See TStringConversion::Init which passes SourceLen+NullOffset to Convert separately.)
static void AppendAsUtf8(TArray<char>& Dest, const TCHAR* Source, int32 SourceLen)
{
	int32 Utf8Len = FPlatformString::ConvertedLength<UTF8CHAR>(Source, SourceLen);
	int32 OldNum = Dest.Num();
	Dest.SetNumUninitialized(OldNum + Utf8Len);
	FPlatformString::Convert(reinterpret_cast<UTF8CHAR*>(Dest.GetData()) + OldNum, Utf8Len, Source, SourceLen);
}

/** Append an RFC 8259 JSON-escaped version of Str into Builder. */
static void AppendJsonEscaped(FStringBuilderBase& Builder, FStringView Str)
{
	for (int32 i = 0; i < Str.Len(); ++i)
	{
		TCHAR Ch = Str[i];
		switch (Ch)
		{
		case '\\': Builder.Append(TEXT("\\\\")); break;
		case '"':  Builder.Append(TEXT("\\\"")); break;
		case '\n': Builder.Append(TEXT("\\n")); break;
		case '\r': Builder.Append(TEXT("\\r")); break;
		case '\t': Builder.Append(TEXT("\\t")); break;
		case '\b': Builder.Append(TEXT("\\b")); break;
		case '\f': Builder.Append(TEXT("\\f")); break;
		default:
			if (Ch < 0x20)
			{
				Builder.Appendf(TEXT("\\u%04x"), static_cast<uint32>(Ch));
			}
			else if (Ch > 0x7E)
			{
				// Non-ASCII: \uXXXX escape to keep JSON as pure ASCII.
				uint32 Codepoint = static_cast<uint32>(Ch);
				if (Codepoint <= 0xFFFF)
				{
					Builder.Appendf(TEXT("\\u%04x"), Codepoint);
				}
				else
				{
					// Surrogate pair for supplementary plane.
					Codepoint -= 0x10000;
					Builder.Appendf(TEXT("\\u%04x\\u%04x"),
						0xD800 + (Codepoint >> 10),
						0xDC00 + (Codepoint & 0x3FF));
				}
			}
			else
			{
				Builder.AppendChar(Ch);
			}
			break;
		}
	}
}

static void AppendJsonString(FStringBuilderBase& Builder, FStringView Str)
{
	Builder.AppendChar(TEXT('"'));
	AppendJsonEscaped(Builder, Str);
	Builder.AppendChar(TEXT('"'));
}



static void AppendCVarJson(FStringBuilderBase& Builder, FStringView Name, IConsoleVariable* CVar)
{
	FString Value = CVar->GetString();
	FString HelpText = CVar->GetHelp();
	int32 NewlineIdx;
	if (HelpText.FindChar('\n', NewlineIdx))
	{
		HelpText.LeftInline(NewlineIdx);
	}
	HelpText.TrimStartAndEndInline();

	Builder.Append(TEXT("\"name\":"));
	AppendJsonString(Builder, Name);
	Builder.Append(TEXT(",\"value\":"));
	AppendJsonString(Builder, Value);
	Builder.Append(TEXT(",\"source\":\""));
	Builder.Append(GetConsoleVariableSetByName(CVar->GetFlags()));
	Builder.Append(TEXT("\",\"help\":"));
	AppendJsonString(Builder, HelpText);
}


class FRemoteConsoleLogCapture : public FOutputDevice
{
public:
	FRemoteConsoleLogCapture() : bSubscribed(false) {}
	~FRemoteConsoleLogCapture()
	{
		Unsubscribe();
	}

	void Subscribe()
	{
		FScopeLock Lock(&DrainLock);
		if (!bSubscribed)
		{
			bSubscribed = true;
			GLog->AddOutputDevice(this);
		}
	}

	void Unsubscribe()
	{
		FScopeLock Lock(&DrainLock);
		if (bSubscribed)
		{
			GLog->RemoveOutputDevice(this);
			bSubscribed = false;
		}
	}

	/** Drain pre-encoded UTF-8 log lines. Returns true if any data was drained. */
	bool DrainInto(TArray<char>& OutBuffer)
	{
		FScopeLock Lock(&QueueLock);
		if (Utf8Queue.Num() == 0)
		{
			return false;
		}
		Swap(OutBuffer, Utf8Queue);
		return true;
	}

protected:
	virtual void Serialize(const TCHAR* V, ELogVerbosity::Type Verbosity, const FName& Category) override
	{
		if (!bSubscribed.load(std::memory_order_acquire))
		{
			return;
		}

		// Build JSON line on the stack first (outside the lock).
		const TCHAR* VerbosityStr = ToString(Verbosity);
		TStringBuilder<512> Builder;
		TStringBuilder<64> CatName;
		Category.AppendString(CatName);

		Builder.Append(TEXT("{\"op\":\"log\",\"cat\":"));
		AppendJsonString(Builder, CatName.ToView());
		Builder.Append(TEXT(",\"v\":\""));
		Builder.Append(VerbosityStr);
		Builder.Append(TEXT("\",\"line\":"));
		AppendJsonString(Builder, FStringView(V));
		Builder.Append(TEXT("}\n"));

		int32 Utf8Len = FPlatformString::ConvertedLength<UTF8CHAR>(Builder.GetData(), Builder.Len());

		FScopeLock Lock(&QueueLock);
		if (Utf8Queue.Num() + Utf8Len <= MaxQueueBytes)
		{
			AppendAsUtf8(Utf8Queue, Builder.GetData(), Builder.Len()); // no embedded NULs; see AppendAsUtf8 comment
		}
	}

private:
	static constexpr int32 MaxQueueBytes = 4 * 1024 * 1024; // 4 MB cap

	FCriticalSection QueueLock;   // Protects Utf8Queue only. Never held across I/O.
	FCriticalSection DrainLock;   // Serializes Subscribe/Unsubscribe (not taken by Serialize).
	TArray<char> Utf8Queue;

	// Atomic: read without lock in Serialize() as fast-path early-out.
	std::atomic<bool> bSubscribed;
};


bool FRemoteConsoleServer::FRecvBuffer::Fill(ITransport& Transport, int32 TimeoutMs)
{
	if (Pos > 0 && Pos < Len)
	{
		FMemory::Memmove(Data.GetData(), Data.GetData() + Pos, Len - Pos);
		Len -= Pos;
		Pos = 0;
	}
	else if (Pos >= Len)
	{
		Pos = 0;
		Len = 0;
	}

	constexpr int32 ReadSize = 4096;
	if (Data.Num() < Len + ReadSize)
	{
		Data.SetNumUninitialized(Len + ReadSize);
	}

	int32 Received = Transport.Recv(Data.GetData() + Len, ReadSize, TimeoutMs);
	if (Received < 0)
	{
		bDisconnected = true;
		return false;
	}
	Len += Received;
	return true;
}

bool FRemoteConsoleServer::FRecvBuffer::ReadLine(TAnsiStringBuilder<512>& OutLine, ITransport& Transport, int32 TimeoutMs)
{
	OutLine.Reset();
	bLineTooLong = false;

	while (true)
	{
		for (int32 i = Pos; i < Len; ++i)
		{
			if (Data[i] == '\n')
			{
				int32 LineEnd = i;
				if (LineEnd > Pos && Data[LineEnd - 1] == '\r')
				{
					LineEnd--;
				}
				OutLine.Append(Data.GetData() + Pos, LineEnd - Pos);
				Pos = i + 1;
				return true;
			}
		}

		if (Pos < Len)
		{
			OutLine.Append(Data.GetData() + Pos, Len - Pos);
			Pos = Len;
		}

		// Guard against unbounded line length (DoS).
		if (OutLine.Len() > MaxLineLength)
		{
			OutLine.Reset();
			bLineTooLong = true;

			int32 DrainBudget = MaxLineLength;
			while (DrainBudget > 0)
			{
				for (int32 i = Pos; i < Len; ++i)
				{
					if (Data[i] == '\n')
					{
						Pos = i + 1;
						return false;
					}
				}
				DrainBudget -= (Len - Pos);
				Pos = Len;
				if (!Fill(Transport, TimeoutMs))
				{
					return false;
				}
				if (Pos >= Len)
				{
					return false;
				}
			}

			bDisconnected = true;
			return false;
		}

		if (!Fill(Transport, TimeoutMs))
		{
			return false;
		}
		if (Pos >= Len)
		{
			return false; // Timeout, no data
		}
	}
}

bool FRemoteConsoleServer::FRecvBuffer::HasData(ITransport& Transport)
{
	if (Pos < Len)
	{
		return true;
	}

	return Fill(Transport, 0) && Pos < Len;
}


bool FRemoteConsoleServer::SendLine(ITransport& Transport, FStringBuilderBase& Builder)
{
	if (Builder.Len() == 0 || Builder.LastChar() != TEXT('\n'))
	{
		Builder.AppendChar(TEXT('\n'));
	}

	FTCHARToUTF8 Utf8(Builder.GetData(), Builder.Len());
	return Transport.Send(Utf8.Get(), Utf8.Length());
}


bool FRemoteConsoleServer::IsHandshake(FAnsiStringView Data)
{
	return Data.Len() > 0 && Data[0] == '{' && Data.Contains(ANSITEXTVIEW("hello"));
}


void FRemoteConsoleServer::HandleConnection(ITransport& Transport)
{
	FRecvBuffer Buffer;
	FCompletionCache CompletionCache;
	FRemoteConsoleLogCapture LogCapture;

	TAnsiStringBuilder<512> Line;
	TArray<char> LogBuffer;
	TStringBuilder<1024> Response;

	// Handshake: read the hello, extract id, respond before subscribing to GLog.
	if (!Buffer.ReadLine(Line, Transport, GReadLineTimeoutMs))
	{
		return;
	}

	{
		// NOTE: initial `hello` handshake handling happens only at this point.
		// Under normal operation afterwards, it will be treated as an error.

		TSharedPtr<FJsonObject> HelloObj;
		TSharedRef<TJsonReader<ANSICHAR>> JsonReader = TJsonReaderFactory<ANSICHAR>::Create(Line.ToString());

		if (!FJsonSerializer::Deserialize(JsonReader, HelloObj) || !HelloObj.IsValid())
		{
			Response.Reset();
			Response.Append(TEXT("{\"op\":\"error\",\"msg\":\"invalid JSON\"}"));
			SendLine(Transport, Response);
			return;
		}
		FString Op;
		if (!HelloObj->TryGetStringField(TEXT("op"), Op) || Op != TEXT("hello"))
		{
			Response.Reset();
			Response.Append(TEXT("{\"op\":\"error\",\"msg\":\"expected hello\"}"));
			SendLine(Transport, Response);
			return;
		}
		int64 Id = 0;
		HelloObj->TryGetNumberField(TEXT("id"), Id);

		Response.Appendf(TEXT("{\"id\":%lld,\"op\":\"hello\",\"version\":1}"), Id);
		if (!SendLine(Transport, Response))
		{
			return;
		}
		Response.Reset();
		Line.Reset();
	}

	// Now subscribe to the log stream -- client is ready to receive.
	LogCapture.Subscribe();

	double LastRecvTime = FPlatformTime::Seconds();

	while (true)
	{
		// 1) Drain and send any pending log lines
		LogBuffer.Reset();
		bool bHadLogs = LogCapture.DrainInto(LogBuffer);
		if (bHadLogs)
		{
			if (!Transport.Send(LogBuffer.GetData(), LogBuffer.Num()))
			{
				break;
			}
		}

		// 2) Check for incoming request (non-blocking)
		if (!Buffer.HasData(Transport))
		{
			if (Buffer.bDisconnected)
			{
				break;
			}
			
			if constexpr (GConnectionTimeoutSeconds > 0.0)
			{
				if ((FPlatformTime::Seconds() - LastRecvTime) > GConnectionTimeoutSeconds)
				{
					break;
				}
			}

			// Skip sleep if logs were just sent -- drain more immediately.
			if (!bHadLogs)
			{
				FPlatformProcess::Sleep(GPollIntervalSeconds);
			}
			continue;
		}

		LastRecvTime = FPlatformTime::Seconds();

		// 3) Read a JSON line (timeout is not a disconnect)
		if (!Buffer.ReadLine(Line, Transport, GReadLineTimeoutMs))
		{
			if (Buffer.bDisconnected)
			{
				break;
			}
			if (Buffer.bLineTooLong)
			{
				Response.Reset();
				Response.Append(TEXT("{\"op\":\"error\",\"msg\":\"line too long\"}"));
				if (!SendLine(Transport, Response))
				{
					break;
				}
			}
			continue;
		}
		if (Line.Len() == 0)
		{
			continue;
		}

		// 4) Parse
		FUTF8ToTCHAR Utf8Decoded(Line.ToString(), Line.Len());
		FStringView JsonView(Utf8Decoded.Get(), Utf8Decoded.Length());
		TSharedPtr<FJsonObject> RequestObj;
		TSharedRef<TJsonReader<>> Reader = TJsonReaderFactory<>::CreateFromView(JsonView);
		if (!FJsonSerializer::Deserialize(Reader, RequestObj) || !RequestObj.IsValid())
		{
			Response.Reset();
			Response.Append(TEXT("{\"op\":\"error\",\"msg\":\"invalid JSON\"}"));
			if (!SendLine(Transport, Response))
			{
				break;
			}
			continue;
		}

		// 5) Dispatch
		int64 Id = 0;
		RequestObj->TryGetNumberField(TEXT("id"), Id);
		FString Op;
		RequestObj->TryGetStringField(TEXT("op"), Op);
		Response.Reset();
		Response.Appendf(TEXT("{\"id\":%lld,\"op\":"), Id);

		if (Op == TEXT("exec"))
		{
			FString Command;
			RequestObj->TryGetStringField(TEXT("cmd"), Command);
			Response.Append(TEXT("\"exec\","));
			// NOTE: remote console is a development-only feature and is expected to be compiled-out in shipping configs.
			// Additionally, the responsibility for access control is on the ITransport implementation.
			HandleExec(Command, Response);
		}
		else if (Op == TEXT("complete"))
		{
			FString Prefix;
			RequestObj->TryGetStringField(TEXT("q"), Prefix);
			int32 Offset = 0, Limit = 50;
			double OffsetVal, LimitVal;
			if (RequestObj->TryGetNumberField(TEXT("offset"), OffsetVal))
			{
				Offset = static_cast<int32>(FMath::Clamp(OffsetVal, 0.0, static_cast<double>(MAX_int32)));
			}
			if (RequestObj->TryGetNumberField(TEXT("limit"), LimitVal))
			{
				Limit = static_cast<int32>(FMath::Clamp(LimitVal, 0.0, static_cast<double>(MAX_int32)));
			}
			if (Limit <= 0)
			{
				Limit = 50;
			}
			if (Limit > 10000)
			{
				Limit = 10000;
			}
			if (Offset < 0)
			{
				Offset = 0;
			}

			Response.Append(TEXT("\"complete\","));
			HandleComplete(Prefix, Offset, Limit, CompletionCache, Response);
		}
		else if (Op == TEXT("getvar"))
		{
			FString VarName;
			RequestObj->TryGetStringField(TEXT("n"), VarName);
			Response.Append(TEXT("\"getvar\","));
			HandleGetVar(VarName, Response);
		}
		else if (Op == TEXT("hello"))
		{
			Response.Append(TEXT("\"error\",\"msg\":\"already connected\""));
		}
		else
		{
			Response.Append(TEXT("\"error\",\"msg\":\"unknown op: "));
			AppendJsonEscaped(Response, Op);
			Response.AppendChar(TEXT('"'));
		}

		Response.AppendChar(TEXT('}'));

		if (!SendLine(Transport, Response))
		{
			break;
		}
	}

	LogCapture.Unsubscribe();
}


void FRemoteConsoleServer::HandleExec(const FString& Command, FStringBuilderBase& Response)
{
	if (GEngine == nullptr)
	{
		Response.Append(TEXT("\"ok\":false,\"error\":\"engine not initialized\""));
		return;
	}

	// Shared control block for game-thread dispatch.
	struct FExecControl
	{
		FEvent* DoneEvent;
		FString Cmd;
		std::atomic<bool> bAbandoned;

		FExecControl(const FString& InCmd)
			: DoneEvent(FPlatformProcess::GetSynchEventFromPool(true))
			, Cmd(InCmd)
			, bAbandoned(false)
		{}

		~FExecControl()
		{
			FPlatformProcess::ReturnSynchEventToPool(DoneEvent);
		}
	};

	TSharedRef<FExecControl> Control = MakeShared<FExecControl>(Command);

	FFunctionGraphTask::CreateAndDispatchWhenReady([Control]
	{
		if (Control->bAbandoned)
		{
			return;
		}
		if (GEngine)
		{
			GEngine->Exec(
				GEngine->GetWorldContexts().Num() > 0 ? GEngine->GetWorldContexts()[0].World() : nullptr,
				*Control->Cmd, *GLog);
		}
		Control->DoneEvent->Trigger();
	}, TStatId(), nullptr, ENamedThreads::GameThread);

	if (!Control->DoneEvent->Wait(GExecTimeoutMs))
	{
		Control->bAbandoned = true;
		Response.Append(TEXT("\"ok\":false,\"error\":\"command timed out (game thread busy)\""));
		return;
	}
	Response.Append(TEXT("\"ok\":true"));
}


void FRemoteConsoleServer::HandleComplete(const FString& Prefix, int32 Offset, int32 Limit,
	FCompletionCache& Cache, FStringBuilderBase& Response)
{
	TArray<FString>* NamesToUse = nullptr;
	TArray<FString> TrieResults;
	int32 TotalMatches = 0;

	int32 CollectEnd = SafeEnd(Offset, Limit);

	// Walk the auto-complete trie on the game thread (it's game-thread-owned).
	{
		struct FTrieWalkControl
		{
			FEvent* DoneEvent;
			TArray<FString> Names;
			int32 Total = 0;
			bool bSuccess = false;

			FTrieWalkControl()
				: DoneEvent(FPlatformProcess::GetSynchEventFromPool(true))
			{}

			~FTrieWalkControl()
			{
				FPlatformProcess::ReturnSynchEventToPool(DoneEvent);
			}
		};

		TSharedRef<FTrieWalkControl> TrieWalk = MakeShared<FTrieWalkControl>();

		FFunctionGraphTask::CreateAndDispatchWhenReady(
			[LowerPrefix = Prefix.ToLower(), Offset, CollectEnd, TrieWalk]
		{
			// Find UConsole on the game thread where it's safe to access.
			UConsole* Console = nullptr;
			if (GEngine)
			{
				for (const FWorldContext& Context : GEngine->GetWorldContexts())
				{
					if (Context.GameViewport && Context.GameViewport->ViewportConsole)
					{
						Console = Context.GameViewport->ViewportConsole;
						break;
					}
				}
			}

			if (!Console)
			{
				TrieWalk->DoneEvent->Trigger();
				return;
			}

			if (!Console->bAutoCompleteLocked
				&& Console->AutoCompleteTree.ChildNodes.Num() == 0)
			{
				Console->BuildRuntimeAutoCompleteList(true);
			}

			if (Console->bAutoCompleteLocked
				|| Console->AutoCompleteTree.ChildNodes.Num() == 0)
			{
				TrieWalk->DoneEvent->Trigger();
				return;
			}

			FAutoCompleteNode* Node = &Console->AutoCompleteTree;
			bool bFound = true;
			for (int32 i = 0; i < LowerPrefix.Len(); ++i)
			{
				int32 Char = static_cast<int32>(LowerPrefix[i]);
				FAutoCompleteNode* Child = nullptr;
				for (FAutoCompleteNode* C : Node->ChildNodes)
				{
					if (C->IndexChar == Char)
					{
						Child = C;
						break;
					}
				}
				if (!Child)
				{
					bFound = false;
					break;
				}
				Node = Child;
			}

			if (bFound)
			{
				TrieWalk->Total = Node->AutoCompleteListIndices.Num();
				int32 Start = FMath::Min(Offset, TrieWalk->Total);
				int32 End = FMath::Min(CollectEnd, TrieWalk->Total);
				for (int32 i = Start; i < End; ++i)
				{
					int32 Idx = Node->AutoCompleteListIndices[i];
					if (Console->AutoCompleteList.IsValidIndex(Idx))
					{
						TrieWalk->Names.Add(Console->AutoCompleteList[Idx].Command);
					}
				}
				TrieWalk->bSuccess = true;
			}

			TrieWalk->DoneEvent->Trigger();
		}, TStatId(), nullptr, ENamedThreads::GameThread);

		TrieWalk->DoneEvent->Wait(GTrieWalkTimeoutMs);

		if (TrieWalk->bSuccess)
		{
			TrieResults = MoveTemp(TrieWalk->Names);
			TotalMatches = TrieWalk->Total;
			NamesToUse = &TrieResults;
		}
	}

	if (!NamesToUse)
	{
		// Fallback: use ForEachConsoleObjectThatStartsWith with cache
		if (!Cache.IsValid(Prefix))
		{
			Cache.SortedNames.Reset();
			Cache.Prefix = Prefix;
			Cache.Timestamp = FPlatformTime::Seconds();

			IConsoleManager::Get().ForEachConsoleObjectThatStartsWith(
				FConsoleObjectVisitor::CreateLambda([&Cache](const TCHAR* Name, IConsoleObject* Obj)
				{
					Cache.SortedNames.Add(FString(Name));
				}),
				*Prefix
			);
			Cache.SortedNames.Sort();
		}

		// Paginate. Trie path already collected the page; fallback has the full set.
		const TArray<FString>* FullList;
		if (!Prefix.Equals(Cache.Prefix, ESearchCase::IgnoreCase))
		{
			for (const FString& Name : Cache.SortedNames)
			{
				if (Name.StartsWith(Prefix, ESearchCase::IgnoreCase))
				{
					TrieResults.Add(Name);
				}
			}
			FullList = &TrieResults;
		}
		else
		{
			FullList = &Cache.SortedNames;
		}

		TotalMatches = FullList->Num();

		int32 PageStart = FMath::Min(Offset, TotalMatches);
		int32 PageEnd = FMath::Min(SafeEnd(Offset, Limit), TotalMatches);
		if (FullList != &TrieResults)
		{
			for (int32 i = PageStart; i < PageEnd; ++i)
			{
				TrieResults.Add((*FullList)[i]);
			}
		}
		else if (PageStart > 0 || PageEnd < TrieResults.Num())
		{
				// Trim to page in-place.
			if (PageEnd < TrieResults.Num())
			{
				TrieResults.SetNum(PageEnd);
			}
			TrieResults.RemoveAt(0, PageStart);
		}
		NamesToUse = &TrieResults;
	}

	// Build JSON results.
	// CVar value reads (GetString/GetInt) must happen on the game thread because some
	// CVar types (e.g. FConsoleVariableBitRef) assert IsInGameThread().
	// We collect the JSON fragments on the game thread, then append them here.
	// Shared state is ref-counted so the lambda is safe even if the wait times out.

	struct FCompleteSharedState
	{
		TArray<FString> Names;
		TArray<FString> JsonEntries;
		FEvent* DoneEvent;
		FCompleteSharedState() : DoneEvent(FPlatformProcess::GetSynchEventFromPool(true)) {}
		~FCompleteSharedState() { FPlatformProcess::ReturnSynchEventToPool(DoneEvent); }
	};

	int32 Sent = 0;

	if (NamesToUse && NamesToUse->Num() > 0)
	{
		TSharedRef<FCompleteSharedState> State = MakeShared<FCompleteSharedState>();
		State->Names = *NamesToUse; // Copy for game-thread lambda

		FFunctionGraphTask::CreateAndDispatchWhenReady(
			[State]
		{
			State->JsonEntries.Reserve(State->Names.Num());
			for (const FString& Name : State->Names)
			{
				TStringBuilder<512> Entry;
				IConsoleVariable* CVar = nullptr;
				if (!Name.Contains(TEXT(" ")))
				{
					IConsoleObject* Obj = IConsoleManager::Get().FindConsoleObject(*Name);
					CVar = Obj ? Obj->AsVariable() : nullptr;
				}
				if (CVar)
				{
					Entry.AppendChar(TEXT('{'));
					AppendCVarJson(Entry, Name, CVar);
					Entry.AppendChar(TEXT('}'));
				}
				else
				{
					Entry.Append(TEXT("{\"name\":"));
					AppendJsonString(Entry, Name);
					Entry.AppendChar(TEXT('}'));
				}
				State->JsonEntries.Add(FString(Entry.ToView()));
			}
			State->DoneEvent->Trigger();
		}, TStatId(), nullptr, ENamedThreads::GameThread);

		if (State->DoneEvent->Wait(GExecTimeoutMs))
		{
			Sent = State->JsonEntries.Num();
			Response.Append(TEXT("\"results\":["));
			for (int32 i = 0; i < Sent; ++i)
			{
				if (i > 0)
				{
					Response.AppendChar(TEXT(','));
				}
				Response.Append(State->JsonEntries[i]);
			}
		}
		else
		{
			Response.Append(TEXT("\"results\":["));
		}
	}
	else
	{
		Response.Append(TEXT("\"results\":["));
	}

	Response.Appendf(TEXT("],\"offset\":%d,\"count\":%d,\"total\":%d"), Offset, Sent, TotalMatches);
}


void FRemoteConsoleServer::HandleGetVar(const FString& VarName, FStringBuilderBase& Response)
{
	// CVar value reads must happen on the game thread (FConsoleVariableBitRef asserts IsInGameThread).
	// Shared state is ref-counted so the lambda is safe even if the wait times out.
	struct FGetVarSharedState
	{
		FString VarName;
		FString ResultJson;
		FEvent* DoneEvent;
		FGetVarSharedState(const FString& InName) : VarName(InName), DoneEvent(FPlatformProcess::GetSynchEventFromPool(true)) {}
		~FGetVarSharedState() { FPlatformProcess::ReturnSynchEventToPool(DoneEvent); }
	};

	TSharedRef<FGetVarSharedState> State = MakeShared<FGetVarSharedState>(VarName);

	FFunctionGraphTask::CreateAndDispatchWhenReady(
		[State]
	{
		TStringBuilder<512> Builder;
		IConsoleVariable* CVar = IConsoleManager::Get().FindConsoleVariable(*State->VarName);
		if (CVar)
		{
			AppendCVarJson(Builder, State->VarName, CVar);
		}
		else
		{
			Builder.Append(TEXT("\"error\":\"CVar '"));
			AppendJsonEscaped(Builder, State->VarName);
			Builder.Append(TEXT("' not found\""));
		}
		State->ResultJson = FString(Builder.ToView());
		State->DoneEvent->Trigger();
	}, TStatId(), nullptr, ENamedThreads::GameThread);

	if (State->DoneEvent->Wait(GExecTimeoutMs))
	{
		Response.Append(State->ResultJson);
	}
	else
	{
		Response.Append(TEXT("\"error\":\"game thread timeout\""));
	}
}


#endif // WITH_REMOTE_CONSOLE_SERVER
