// Copyright Epic Games, Inc. All Rights Reserved.

#include "Engine/RendererSettings.h"
#include "GPUSkinVertexFactory.h"
#include "ColorManagement/ColorSpace.h"
#include "HAL/PlatformFile.h"
#include "HDRHelper.h"
#include "SceneManagement.h"
#include "Misc/App.h"
#include "Misc/Paths.h"
#include "RenderUtils.h"
#include "UObject/UnrealType.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RendererSettings)

#if WITH_EDITOR
#include "Editor/EditorEngine.h"
#include "Misc/App.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/MessageDialog.h"
#include "HAL/PlatformFileManager.h"

#if PLATFORM_WINDOWS || PLATFORM_LINUX || PLATFORM_MAC
#include "Framework/Docking/TabManager.h"
#include "Framework/Notifications/NotificationManager.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "DataDrivenShaderPlatformInfo.h"
#endif

/** The editor object. */
extern UNREALED_API class UEditorEngine* GEditor;
#endif // #if WITH_EDITOR

#define LOCTEXT_NAMESPACE "RendererSettings"

namespace EDefaultBackBufferPixelFormat
{
	EPixelFormat Convert2PixelFormat(EDefaultBackBufferPixelFormat::Type InDefaultBackBufferPixelFormat)
	{
		const int32 ValidIndex = FMath::Clamp((int32)InDefaultBackBufferPixelFormat, 0, (int32)DBBPF_MAX - 1);
		static EPixelFormat SPixelFormat[] = { PF_B8G8R8A8, PF_B8G8R8A8, PF_FloatRGBA, PF_FloatRGBA, PF_A2B10G10R10 };
		return SPixelFormat[ValidIndex];
	}

	int32 NumberOfBitForAlpha(EDefaultBackBufferPixelFormat::Type InDefaultBackBufferPixelFormat)
	{
		switch (InDefaultBackBufferPixelFormat)
		{
			case DBBPF_A16B16G16R16_DEPRECATED:
			case DBBPF_B8G8R8A8:
			case DBBPF_FloatRGB_DEPRECATED:
			case DBBPF_FloatRGBA:
				return 8;
			case DBBPF_A2B10G10R10:
				return 2;
			default:
				return 0;
		}
	}

	EDefaultBackBufferPixelFormat::Type FromInt(int32 InDefaultBackBufferPixelFormat)
	{
		const int32 ValidIndex = FMath::Clamp(InDefaultBackBufferPixelFormat, 0, (int32)DBBPF_MAX - 1);
		static EDefaultBackBufferPixelFormat::Type SPixelFormat[] = { DBBPF_B8G8R8A8, DBBPF_B8G8R8A8, DBBPF_FloatRGBA, DBBPF_FloatRGBA, DBBPF_A2B10G10R10 };
		return SPixelFormat[ValidIndex];
	}
}

URendererSettings::URendererSettings(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	SectionName = TEXT("Rendering");
	TranslucentSortAxis = FVector(0.0f, -1.0f, 0.0f);
	bSupportStationarySkylight = true;
	bSupportPointLightWholeSceneShadows = true;
	MorphTargetMaxBlendWeight = 5.f;
	bSupportSkyAtmosphere = true;
	SkinningShaderSupport = ESkinningShaderSupportMode::InlineShadersOnly;
	GPUSimulationTextureSizeX = 1024;
	GPUSimulationTextureSizeY = 1024;
	bEnableRayTracing = 0;
	bUseHardwareRayTracingForLumen = 0;
	bEnableMegaLights = 0;
	bEnableRayTracingShadows = 0;
	bEnablePathTracing = 0;
	bEnableRayTracingTextureLOD = 0;
	DefaultBoneInfluenceLimit = 0;
	MaxSkinBones = FGPUBaseSkinVertexFactory::GHardwareMaxGPUSkinBones;
	WorkingColorSpaceChoice = EWorkingColorSpace::sRGB;
	RedChromaticityCoordinate = FVector2D::ZeroVector;
	GreenChromaticityCoordinate = FVector2D::ZeroVector;
	BlueChromaticityCoordinate = FVector2D::ZeroVector;
	WhiteChromaticityCoordinate = FVector2D::ZeroVector;
	bUseLegacyLuminanceFactors = false;
	bEnableVirtualTextureOpacityMask = false;

#if WITH_EDITOR
	if (IsTemplate())
	{
		PreInitPropertiesFixup();
	}
#endif // #if WITH_EDITOR
}

void URendererSettings::PostInitProperties()
{
	Super::PostInitProperties();
	
	SanitizeReflectionCaptureResolution();

	UpdateWorkingColorSpaceAndChromaticities();


#if WITH_EDITOR
	CheckForMissingShaderModels();

	if (IsTemplate())
	{
		ImportConsoleVariableValues();
	}
#endif // #if WITH_EDITOR
}

#if WITH_EDITOR
void URendererSettings::PreEditChange(FProperty* PropertyAboutToChange)
{
	Super::PreEditChange(PropertyAboutToChange);

	PreEditReflectionCaptureResolution = ReflectionCaptureResolution;
}

void UpdateDependentPropertyInConfigFile(URendererSettings* RendererSettings, FName PropertyName)
{
	// Duplicate SSettingsEditor::NotifyPostChange functionality to make sure we are able to write to the file before we modify the ini
	// Only duplicating the MakeWriteable behavior, SSettingsEditor::NotifyPostChange will be called after this on the main property and handle source control
	//@todo - unify with SSettingsEditor::NotifyPostChange 

	check(RendererSettings->GetClass()->HasAnyClassFlags(CLASS_DefaultConfig));
	
	FString RelativePath = RendererSettings->GetDefaultConfigFilename();
	FString FullPath = FPaths::ConvertRelativePathToFull(RelativePath);

	const bool bIsWriteable = !FPlatformFileManager::Get().GetPlatformFile().IsReadOnly(*FullPath);

	if (!bIsWriteable)
	{
		FPlatformFileManager::Get().GetPlatformFile().SetReadOnly(*FullPath, false);
	}

	for (TFieldIterator<FProperty> PropIt(RendererSettings->GetClass()); PropIt; ++PropIt)
	{
		FProperty* Property = *PropIt;
		if (Property->GetFName() == PropertyName)
		{
			RendererSettings->UpdateSinglePropertyInConfigFile(Property, RendererSettings->GetDefaultConfigFilename());
		}
	}

	// Restore original state for source control
	if (!bIsWriteable)
	{
		FPlatformFileManager::Get().GetPlatformFile().SetReadOnly(*FullPath, true);
	}
}

void URendererSettings::PreInitPropertiesFixup()
{
	int32 PropagateAlphaIntValue = 0;
	if (GConfig->GetInt(TEXT("/Script/Engine.RendererSettings"), TEXT("r.PostProcessing.PropagateAlpha"), PropagateAlphaIntValue, GEngineIni))
	{
		constexpr int32 LegacyAllowThroughTonemapper = 2;
		if (PropagateAlphaIntValue == LegacyAllowThroughTonemapper)
		{
			UE_LOGF(LogEngine, Warning, "r.PostProcessing.PropagateAlpha config value under /Script/Engine.RendererSettings was automatically converted from 2 to True, please version the change.");

			bEnableAlphaChannelInPostProcessing = true;
			UpdateDependentPropertyInConfigFile(this, GET_MEMBER_NAME_CHECKED(URendererSettings, bEnableAlphaChannelInPostProcessing));
		}
	}
}

void URendererSettings::FixAntiAliasingOnShadingPathChange(FPropertyChangedEvent& PropertyChangedEvent)
{
	if (PropertyChangedEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(URendererSettings, MobileShadingPath))
	{
		if (MobileShadingPath == EMobileShadingPath::Forward)
		{
			// Nop
		}
		else // When we disable forward shading we need to change AA method
		{
			if (MobileAntiAliasing == EMobileAntiAliasingMethod::MSAA)
			{
				MobileAntiAliasing = EMobileAntiAliasingMethod::None;
				UE_LOGF(LogTemp, Warning, "Disabling Mobile AA because MSAA is not compatible with Mobile Deferred Shading");
			}
		}
		UpdateDependentPropertyInConfigFile(this, GET_MEMBER_NAME_CHECKED(URendererSettings, MobileAntiAliasing));
		ExportValuesToConsoleVariables(GetClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(URendererSettings, MobileAntiAliasing)));
	}
	// On desktop going from deferred to forward we force MSAA. After that, the AA method is blocked by the edit condition in metadata.
	else if (PropertyChangedEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(URendererSettings, bForwardShading))
	{
		if (bForwardShading)
		{
			DefaultFeatureAntiAliasing = EAntiAliasingMethod::AAM_MSAA;
		}
		else // When we disable forward shading we need to change AA method
		{
			DefaultFeatureAntiAliasing = EAntiAliasingMethod::AAM_None;
			UE_LOGF(LogTemp, Warning, "Disabling AA because MSAA is not compatible with Deferred Shading");
		}
		UpdateDependentPropertyInConfigFile(this, GET_MEMBER_NAME_CHECKED(URendererSettings, DefaultFeatureAntiAliasing));
		ExportValuesToConsoleVariables(GetClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(URendererSettings, DefaultFeatureAntiAliasing)));
	}

	// MSAA can't be selected if deferred shading is active
	if (PropertyChangedEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(URendererSettings, MobileAntiAliasing))
	{
		if (MobileAntiAliasing == EMobileAntiAliasingMethod::MSAA && MobileShadingPath == EMobileShadingPath::Deferred)
		{
			MobileAntiAliasing = EMobileAntiAliasingMethod::None;
			UpdateDependentPropertyInConfigFile(this, GET_MEMBER_NAME_CHECKED(URendererSettings, MobileAntiAliasing));
			ExportValuesToConsoleVariables(GetClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(URendererSettings, MobileAntiAliasing)));

			UE_LOGF(LogTemp, Error, "MSAA can't be used with Mobile Deferred Rendering. Resetting Mobile AA to None.");
		}
	}
	else if (PropertyChangedEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(URendererSettings, DefaultFeatureAntiAliasing))
	{
		if (DefaultFeatureAntiAliasing == EAntiAliasingMethod::AAM_MSAA && !bForwardShading)
		{
			DefaultFeatureAntiAliasing = EAntiAliasingMethod::AAM_None;
			UpdateDependentPropertyInConfigFile(this, GET_MEMBER_NAME_CHECKED(URendererSettings, DefaultFeatureAntiAliasing));
			ExportValuesToConsoleVariables(GetClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(URendererSettings, DefaultFeatureAntiAliasing)));
			
			UE_LOGF(LogTemp, Error, "MSAA can't be used with Deferred Rendering. Resetting AA to None.");
		}
	}
}

void URendererSettings::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	SanitizeReflectionCaptureResolution();

	if (PropertyChangedEvent.Property)
	{
		// round up GPU sim texture sizes to nearest power of two, and constrain to sensible values
		if ( PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(URendererSettings, GPUSimulationTextureSizeX) 
			|| PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(URendererSettings, GPUSimulationTextureSizeY) ) 
		{
			static const int32 MinGPUSimTextureSize = 32;
			static const int32 MaxGPUSimTextureSize = 8192;
			GPUSimulationTextureSizeX = FMath::RoundUpToPowerOfTwo( FMath::Clamp(GPUSimulationTextureSizeX, MinGPUSimTextureSize, MaxGPUSimTextureSize) );
			GPUSimulationTextureSizeY = FMath::RoundUpToPowerOfTwo( FMath::Clamp(GPUSimulationTextureSizeY, MinGPUSimTextureSize, MaxGPUSimTextureSize) );
		}

		if (PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(URendererSettings, bEnableRayTracing)
			&& bEnableRayTracing 
			&& SkinningShaderSupport==ESkinningShaderSupportMode::InlineShadersOnly)
		{
			FString FullPath = FPaths::ConvertRelativePathToFull(GetDefaultConfigFilename());
			FPlatformFileManager::Get().GetPlatformFile().SetReadOnly(*FullPath, false);

			if (FMessageDialog::Open(EAppMsgType::YesNo, LOCTEXT("Skin Cache Disabled", "Ray Tracing requires enabling skin cache. Do you want to automatically enable skin cache now?")) == EAppReturnType::Yes)
			{
				SkinningShaderSupport = ESkinningShaderSupportMode::InlineAndSkinCacheShaders;
				UpdateDependentPropertyInConfigFile(this, GET_MEMBER_NAME_CHECKED(URendererSettings, SkinningShaderSupport));
			}
			else
			{
				bEnableRayTracing = 0;
				UpdateDependentPropertyInConfigFile(this, GET_MEMBER_NAME_CHECKED(URendererSettings, bEnableRayTracing));
			}
		}

		if (PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(URendererSettings, SkinningShaderSupport)
			&& bEnableRayTracing
			&& SkinningShaderSupport == ESkinningShaderSupportMode::InlineShadersOnly)
		{
			FMessageDialog::Open(EAppMsgType::Ok, LOCTEXT("Ray Tracing Requires Skin Cache", "Ray Tracing is enabled and requires the use of skin cache shaders. The Skin Cache Shader Support has been reenabled."));

			SkinningShaderSupport = ESkinningShaderSupportMode::InlineAndSkinCacheShaders;
			
			UpdateDependentPropertyInConfigFile(this, GET_MEMBER_NAME_CHECKED(URendererSettings, SkinningShaderSupport));
		}

		if (PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(URendererSettings, DynamicGlobalIllumination) 
			&& DynamicGlobalIllumination == EDynamicGlobalIlluminationMethod::Lumen)
		{
			if (Reflections != EReflectionMethod::Lumen)
			{
				FMessageDialog::Open(EAppMsgType::Ok, LOCTEXT("Lumen Reflections automatically enabled", "Lumen Reflections are designed to work with Lumen Global Illumination, and have been automatically enabled."));

				Reflections = EReflectionMethod::Lumen;

				IConsoleVariable* CVar = IConsoleManager::Get().FindConsoleVariable(TEXT("r.ReflectionMethod"));
				CVar->Set((int32)Reflections, ECVF_SetByProjectSetting);

				UpdateDependentPropertyInConfigFile(this, GET_MEMBER_NAME_CHECKED(URendererSettings, Reflections));
			}

			if (!bGenerateMeshDistanceFields)
			{
				FMessageDialog::Open(EAppMsgType::Ok, LOCTEXT("'Generate Mesh Distance Fields' automatically enabled", "Lumen Global Illumination requires 'Generate Mesh Distance Fields'.  This has been enabled automatically, and requires an editor restart."));

				bGenerateMeshDistanceFields = true;

				UpdateDependentPropertyInConfigFile(this, GET_MEMBER_NAME_CHECKED(URendererSettings, bGenerateMeshDistanceFields));
			}
		}

		if (PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(URendererSettings, SubstrateDebugAdvancedVisualizationShaders)
			&& SubstrateDebugAdvancedVisualizationShaders)
		{
			if (FMessageDialog::Open(EAppMsgType::YesNo, LOCTEXT("Substrate advanced visualization shaders", "This will make the rendering slower and allocate more memory.\nPlease do not check-in.\nInstead you could add `r.Substrate.Debug.AdvancedVisualizationShaders=1` to your ConsoleVariables.ini.\nAre you sure you want to enable that feature now?")) == EAppReturnType::No)
			{
				SubstrateDebugAdvancedVisualizationShaders = false;
				UpdateDependentPropertyInConfigFile(this, GET_MEMBER_NAME_CHECKED(URendererSettings, SubstrateDebugAdvancedVisualizationShaders));
			}
		}

		if (PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(URendererSettings, VirtualTextureTileSize))
		{
			VirtualTextureTileSize = FMath::RoundUpToPowerOfTwo(VirtualTextureTileSize);
		}

		if (PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(URendererSettings, VirtualTextureTileBorderSize))
		{
			VirtualTextureTileBorderSize = FMath::RoundUpToPowerOfTwo(VirtualTextureTileBorderSize);
		}

		if ((PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(URendererSettings, bSupportSkyAtmosphere)))
		{
			if (!bSupportSkyAtmosphere)
			{
				bSupportSkyAtmosphereAffectsHeightFog = 0; // Always disable sky affecting height fog if sky is disabled.
			}
		}

		if ((PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(URendererSettings, MobileShadingPath)))
		{
			if(MobileShadingPath.GetValue() == 1)
			{
				bMobilePostProcessing = 1;
				UpdateDependentPropertyInConfigFile(this, GET_MEMBER_NAME_CHECKED(URendererSettings, bMobilePostProcessing));
			}
		}

		if ((PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(URendererSettings, bEnableSubstrate)))
		{
			if (bEnableSubstrate)
			{
				const EAppReturnType::Type Choice = FMessageDialog::Open(EAppMsgType::OkCancel, LOCTEXT("EnableSubstrateWarning", "Warning: Be aware that any materials saved when Substrate is enabled won't be rendered correctly if Substrate is disabled later on."));
				if (Choice == EAppReturnType::Cancel)
				{
					bEnableSubstrate = false;
				}
			}
			else
			{
				const EAppReturnType::Type Choice = FMessageDialog::Open(EAppMsgType::OkCancel, LOCTEXT("DisableSubstrateWarning", "Warning: Disabling Substrate will cause all materials authored with Substrate nodes to appear incorrect, as they are not backward compatible with the legacy root node and GBuffer format."));
				if (Choice == EAppReturnType::Cancel)
				{
					bEnableSubstrate = true;
				}
			}
		}

		if (PropertyChangedEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(URendererSettings, bEnableAlphaChannelInPostProcessing))
		{
			// Auto-disable primitive alpha holdout if alpha output is disabled.
			bDeferredSupportPrimitiveAlphaHoldout &= bEnableAlphaChannelInPostProcessing;
		}

		ExportValuesToConsoleVariables(PropertyChangedEvent.Property);

		if (PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(URendererSettings, ReflectionCaptureResolution) &&
			PropertyChangedEvent.ChangeType != EPropertyChangeType::Interactive)
		{
			if (GEditor != nullptr)
			{
				if (GWorld != nullptr && GWorld->GetFeatureLevel() == ERHIFeatureLevel::ES3_1)
				{
					// When we feature change from SM5 to ES31 we call BuildReflectionCapture if we have Unbuilt Reflection Components, so no reason to call it again here
					// This is to make sure that we have valid data for Mobile Preview.

					// ES31->SM5 to be able to capture
					GEditor->TogglePreviewPlatform();
					// SM5->ES31 BuildReflectionCaptures are triggered here on callback
					GEditor->TogglePreviewPlatform();
				}
				else
				{
					GEditor->BuildReflectionCaptures();
				}
			}
		}

		if (PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(URendererSettings, WorkingColorSpaceChoice))
		{
			UpdateWorkingColorSpaceAndChromaticities();

			UpdateDependentPropertyInConfigFile(this, GET_MEMBER_NAME_CHECKED(URendererSettings, RedChromaticityCoordinate));
			UpdateDependentPropertyInConfigFile(this, GET_MEMBER_NAME_CHECKED(URendererSettings, GreenChromaticityCoordinate));
			UpdateDependentPropertyInConfigFile(this, GET_MEMBER_NAME_CHECKED(URendererSettings, BlueChromaticityCoordinate));
			UpdateDependentPropertyInConfigFile(this, GET_MEMBER_NAME_CHECKED(URendererSettings, WhiteChromaticityCoordinate));
		}

		if ((PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(URendererSettings, RedChromaticityCoordinate)) ||
			(PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(URendererSettings, GreenChromaticityCoordinate)) ||
			(PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(URendererSettings, BlueChromaticityCoordinate)) ||
			(PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(URendererSettings, WhiteChromaticityCoordinate)))
		{
			UpdateWorkingColorSpaceAndChromaticities();
		}

		if (PropertyChangedEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(URendererSettings, ShadowMapMethod))
		{
			CheckForMissingShaderModels();
		}

		if (PropertyChangedEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(URendererSettings, bSupportHardwareVariableRateShading))
		{
			if (!bSupportHardwareVariableRateShading)
			{
				FoveationLevel = EFixedFoveationLevels::Disabled;
				UpdateDependentPropertyInConfigFile(this, GET_MEMBER_NAME_CHECKED(URendererSettings, FoveationLevel));
			}
		}

		// When SLW depth prepass is disabled, auto-correct velocity to base pass mode
		if (PropertyChangedEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(URendererSettings, bWaterSingleLayerDepthPrepass) 
			|| PropertyChangedEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(URendererSettings, SingleLayerWaterVelocityOutputPass))
		{
			static IConsoleVariable* DepthPrepassCVar = IConsoleManager::Get().FindConsoleVariable(TEXT("r.Water.SingleLayer.DepthPrepass"));
			static IConsoleVariable* VelocityOutputPassCVar = IConsoleManager::Get().FindConsoleVariable(TEXT("r.Water.SingleLayer.VelocityOutputPass"));

			if (DepthPrepassCVar && VelocityOutputPassCVar)
			{
				const bool bDepthPrepass = DepthPrepassCVar->GetInt() != 0;
				const int32 VelocityMode = VelocityOutputPassCVar->GetInt();

				if (!bDepthPrepass && VelocityMode == /* DepthPrepass */1)
				{
					SingleLayerWaterVelocityOutputPass = ESingleLayerWaterVelocityOutputPass::BasePass;
					UpdateDependentPropertyInConfigFile(this, GET_MEMBER_NAME_CHECKED(URendererSettings, SingleLayerWaterVelocityOutputPass));
					FProperty* VelocityProp = GetClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(URendererSettings, SingleLayerWaterVelocityOutputPass));
					if (VelocityProp)
					{
						ExportValuesToConsoleVariables(VelocityProp);
					}
					UE_LOGF(LogTemp, Warning, "r.Water.SingleLayer.VelocityOutputPass set to 2 (base pass) because DepthPrepass is disabled.");
				}
			}
		}

		FixAntiAliasingOnShadingPathChange(PropertyChangedEvent);
	}
}

bool URendererSettings::CanEditChange(const FProperty* InProperty) const
{
	if (!Super::CanEditChange(InProperty))
	{
		return false;
	}

	// the following settings can only be edited if ray tracing is enabled
	if ((InProperty->GetFName() == GET_MEMBER_NAME_CHECKED(URendererSettings, bEnablePathTracing)) ||
		(InProperty->GetFName() == GET_MEMBER_NAME_CHECKED(URendererSettings, bEnableMegaLights)) ||
		(InProperty->GetFName() == GET_MEMBER_NAME_CHECKED(URendererSettings, bEnableRayTracingShadows)) ||
		(InProperty->GetFName() == GET_MEMBER_NAME_CHECKED(URendererSettings, bEnableRayTracingTextureLOD)))
	{
		return bEnableRayTracing;
	}

	if ((InProperty->GetFName() == GET_MEMBER_NAME_CHECKED(URendererSettings, bSupportSkyAtmosphereAffectsHeightFog)))
	{
		return bSupportSkyAtmosphere;
	}

	if (InProperty->GetFName() == GET_MEMBER_NAME_CHECKED(URendererSettings, DynamicGlobalIllumination)
		|| InProperty->GetFName() == GET_MEMBER_NAME_CHECKED(URendererSettings, Reflections)
		|| InProperty->GetFName() == GET_MEMBER_NAME_CHECKED(URendererSettings, ShadowMapMethod))
	{
		return !bForwardShading;
	}

	if ((InProperty->GetFName() == GET_MEMBER_NAME_CHECKED(URendererSettings, bMobilePostProcessing)))
	{
		return MobileShadingPath.GetValue() == 0;
	}

	if ((InProperty->GetFName() == GET_MEMBER_NAME_CHECKED(URendererSettings, DefaultBackBufferPixelFormat)))
	{
		return !IsHDREnabled();
	}

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	// only allow changing ExtendDefaultLuminanceRange if it was disabled.
	// we don't want new projects disabling this setting but still allow existing projects to enable it.
	static bool bCanEditExtendDefaultLuminanceRange = !bExtendDefaultLuminanceRangeInAutoExposureSettings;

	if ((InProperty->GetFName() == GET_MEMBER_NAME_CHECKED(URendererSettings, bExtendDefaultLuminanceRangeInAutoExposureSettings)))
	{
		return bCanEditExtendDefaultLuminanceRange;
	}
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	return true;
}

void URendererSettings::CheckForMissingShaderModels()
{
	static bool bWarnedAboutMissingShaderModel = false;
	// Don't show the SM6 toasts on non-Windows/Linux platforms to avoid confusion around platform requirements.
#if PLATFORM_WINDOWS || PLATFORM_LINUX || PLATFORM_MAC
	if (GIsEditor && (ShadowMapMethod == EShadowMapMethod::VirtualShadowMaps || bEnableRayTracing) && !bWarnedAboutMissingShaderModel)
	{
		auto CopySM6Format = [](const TCHAR* ShaderFormatName, const TArray<FString>& SrcArray, TArray<FString>& DstArray)
		{
			if (SrcArray.Contains(ShaderFormatName))
			{
				DstArray.AddUnique(ShaderFormatName);
			}
		};

		TArray<FString> D3D11TargetedShaderFormats;
		TArray<FString> D3D12TargetedShaderFormats;
		TArray<FString> WindowsVulkanTargetedShaderFormats;
		TArray<FString> WindowsTargetedRHIs;
		TArray<FString> LinuxVulkanTargetedShaderFormats;
		TArray<FString> LinuxTargetedRHIs;
		TArray<FString> MacTargetedShaderFormats;
		TArray<FString> MacTargetedRHIs;

#if PLATFORM_WINDOWS
		// Gather all Windows shader format settings
		{
			GConfig->GetArray(TEXT("/Script/WindowsTargetPlatform.WindowsTargetSettings"), TEXT("TargetedRHIs"), WindowsTargetedRHIs, GEngineIni);

			// If using Vulkan in Windows, warn about Vulkan settings
			if (IsVulkanPlatform(GMaxRHIShaderPlatform))
			{
				GConfig->GetArray(TEXT("/Script/WindowsTargetPlatform.WindowsTargetSettings"), TEXT("VulkanTargetedShaderFormats"), WindowsVulkanTargetedShaderFormats, GEngineIni);
				CopySM6Format(TEXT("SF_VULKAN_SM6"), WindowsTargetedRHIs, WindowsVulkanTargetedShaderFormats);
			}
			else
			{
				GConfig->GetArray(TEXT("/Script/WindowsTargetPlatform.WindowsTargetSettings"), TEXT("D3D11TargetedShaderFormats"), D3D11TargetedShaderFormats, GEngineIni);
				GConfig->GetArray(TEXT("/Script/WindowsTargetPlatform.WindowsTargetSettings"), TEXT("D3D12TargetedShaderFormats"), D3D12TargetedShaderFormats, GEngineIni);
				CopySM6Format(TEXT("PCD3D_SM6"), WindowsTargetedRHIs, D3D12TargetedShaderFormats);
			}
		}
#elif PLATFORM_LINUX
		// Gather all Linux shader format settings
		GConfig->GetArray(TEXT("/Script/LinuxTargetPlatform.LinuxTargetSettings"), TEXT("VulkanTargetedShaderFormats"), LinuxVulkanTargetedShaderFormats, GEngineIni);
		GConfig->GetArray(TEXT("/Script/LinuxTargetPlatform.LinuxTargetSettings"), TEXT("TargetedRHIs"), LinuxTargetedRHIs, GEngineIni);
		CopySM6Format(TEXT("SF_VULKAN_SM6"), LinuxTargetedRHIs, LinuxVulkanTargetedShaderFormats);
#elif PLATFORM_MAC
		// TODO: Gather all Mac shader format settings
		GConfig->GetArray(TEXT("/Script/MacTargetPlatform.MacTargetSettings"), TEXT("TargetedRHIs"), MacTargetedRHIs, GEngineIni);
		CopySM6Format(TEXT("SF_METAL_SM6"), MacTargetedRHIs, MacTargetedShaderFormats);
#endif

		const bool bProjectUsesD3D = (D3D11TargetedShaderFormats.Num() + D3D12TargetedShaderFormats.Num()) > 0;
		const bool bProjectMissingD3DSM6 = (bProjectUsesD3D && !D3D12TargetedShaderFormats.Contains(TEXT("PCD3D_SM6")));

		const bool bProjectUsesWindowsVulkan = (WindowsVulkanTargetedShaderFormats.Num() > 0);
		const bool bProjectMissingWindowsVulkanSM6 = (bProjectUsesWindowsVulkan && !WindowsVulkanTargetedShaderFormats.Contains(TEXT("SF_VULKAN_SM6")));

		const bool bProjectUsesLinuxVulkan = (LinuxTargetedRHIs.Num() > 0) || (LinuxVulkanTargetedShaderFormats.Num() > 0);
		const bool bProjectMissingLinuxVulkanSM6 = (bProjectUsesLinuxVulkan && !LinuxVulkanTargetedShaderFormats.Contains(TEXT("SF_VULKAN_SM6")));
		
		const bool bProjectUsesMetal = (MacTargetedRHIs.Num() > 0) || (MacTargetedShaderFormats.Num() > 0);
		const bool bProjectMissingMacSM6 = (bProjectUsesMetal && !MacTargetedShaderFormats.Contains(TEXT("SF_METAL_SM6")));

		if (bProjectMissingD3DSM6 || bProjectMissingWindowsVulkanSM6 || bProjectMissingLinuxVulkanSM6 || bProjectMissingMacSM6)
		{
			bWarnedAboutMissingShaderModel = true;

			auto DismissNotification = [This = TStrongObjectPtr(this)]
			{
				if (URendererSettings* Settings = This.Get())
				{
					if (TSharedPtr<SNotificationItem> NotificationPin = Settings->ShaderModelNotificationPtr.Pin())
					{
						NotificationPin->SetCompletionState(SNotificationItem::CS_None);
						NotificationPin->ExpireAndFadeout();
						Settings->ShaderModelNotificationPtr.Reset();
					}
				}
			};

			auto OpenProjectSettings = []()
			{
				FGlobalTabmanager::Get()->TryInvokeTab(FName("ProjectSettings"));
			};

			FNotificationInfo Info(LOCTEXT("NeedProjectSettings", "Missing Project Settings!"));
			Info.bFireAndForget = false;
			Info.FadeOutDuration = 0.0f;
			Info.ExpireDuration = 0.0f;
			Info.WidthOverride = FOptionalSize();

			Info.ButtonDetails.Add(FNotificationButtonInfo(
				LOCTEXT("GuidelineDismiss", "Dismiss"),
				LOCTEXT("GuidelineDismissTT", "Dismiss this notification."),
				FSimpleDelegate::CreateLambda(DismissNotification),
				SNotificationItem::CS_None));

			Info.Text = LOCTEXT("NeedProjectSettings", "Missing Project Settings!");
			Info.HyperlinkText = LOCTEXT("ProjectSettingsHyperlinkText", "Open Project Settings");
			Info.Hyperlink = FSimpleDelegate::CreateLambda(OpenProjectSettings);
	
			const FText FeaturesNeedSM6[] =
			{
				LOCTEXT("VirtualShadowMapsNeedsSM6", "Shader Model 6 (SM6) is required to use Virtual Shadow Maps."),
				LOCTEXT("RayTracingNeedsSM6", "Shader Model 6 (SM6) is required to use Ray Tracing."),
				LOCTEXT("VirtualShadowMapsAndRayTracingNeedsSM6", "Shader Model 6 (SM6) is required to use Virtual Shadow Maps and Ray Tracing.")
			};

			const uint32 FeatureNeedsSM6Index = (ShadowMapMethod == EShadowMapMethod::VirtualShadowMaps && bEnableRayTracing) ? 2 : (bEnableRayTracing ? 1 : 0);

			if (bProjectMissingD3DSM6)
			{
				const FText FeatureWontWork[] =
				{
					LOCTEXT("VirtualShadowMapsWontWork", "Virtual shadow maps will not work until this is enabled."),
					LOCTEXT("RayTracingWontWork", "Ray tracing will not work until this is enabled."),
					LOCTEXT("VirtualShadowMapsAndRayTracingWontWork", "Virtual shadow maps and ray tracing will not work until this is enabled."),
				};

				Info.SubText = FText::Format(
					LOCTEXT("VirtualShadowMapsNeedsSM6Setting", "{0} Please enable this in:\n  Project Settings -> Platforms -> Windows -> D3D12 Targeted Shader Formats\n{1}"),
					FeaturesNeedSM6[FeatureNeedsSM6Index],
					FeatureWontWork[FeatureNeedsSM6Index]
				);
			}
			else if (bProjectMissingWindowsVulkanSM6)
			{
				const FText FeatureWontWorkWindowsVulkan[] =
				{
					LOCTEXT("VirtualShadowMapsWontWorkWindowsVulkan", "Virtual shadow maps will not work in Vulkan on Windows until this is enabled."),
					LOCTEXT("RayTracingWontWorkWindowsVulkan", "Ray tracing will not work in Vulkan on Windows until this is enabled."),
					LOCTEXT("VirtualShadowMapsAndRayTracingWontWorkWindowsVulkan", "Virtual shadow maps and ray tracing will not work in Vulkan on Windows until this is enabled."),
				};

				Info.SubText = FText::Format(
					LOCTEXT("VirtualShadowMapsNeedsVulkanSM6WindowsSetting", "{0} Please enable this in:\n  Project Settings -> Platforms -> Windows -> Vulkan Targeted Shader Formats\n{1}"),
					FeaturesNeedSM6[FeatureNeedsSM6Index],
					FeatureWontWorkWindowsVulkan[FeatureNeedsSM6Index]
				);
			}
			else if (bProjectMissingLinuxVulkanSM6)
			{
				const FText FeatureWontWorkLinuxVulkan[] =
				{
					LOCTEXT("VirtualShadowMapsWontWorkLinuxVulkan", "Virtual shadow maps will not work in Vulkan on Linux until this is enabled.."),
					LOCTEXT("RayTracingWontWorkLinuxVulkan", "Ray tracing will not work in Vulkan on Linux until this is enabled."),
					LOCTEXT("VirtualShadowMapsAndRayTracingWontWorkLinuxVulkan", "Virtual shadow maps and ray tracing will not work in Vulkan on Linux until this is enabled."),
				};

				Info.SubText = FText::Format(
					LOCTEXT("VirtualShadowMapsNeedsVulkanSM6LinuxSetting", "{0} Please enable this in:\n  Project Settings -> Platforms -> Linux -> Targeted RHIs\n{1}"),
					FeaturesNeedSM6[FeatureNeedsSM6Index],
					FeatureWontWorkLinuxVulkan[FeatureNeedsSM6Index]
				);
			}
			else if(bProjectMissingMacSM6)
			{
				const FText FeatureWontWorkMac[] =
				{
					LOCTEXT("VirtualShadowMapsWontWorkMac", "Virtual shadow maps will not work on Mac until this is enabled.."),
					LOCTEXT("RayTracingWontWorkMac", "Ray tracing will not work on Mac until this is enabled.."),
					LOCTEXT("VirtualShadowMapsAndRayTracingWontWorkMac", "Virtual shadow maps and ray tracing will not work in Mac until this is enabled."),
				};

				Info.SubText = FText::Format(
					LOCTEXT("VirtualShadowMapsNeedsMacSM6Setting", "{0} Please enable this in:\n  Project Settings -> Platforms -> Mac -> Targeted RHIs\n{1}"),
					FeaturesNeedSM6[FeatureNeedsSM6Index],
					FeatureWontWorkMac[FeatureNeedsSM6Index]
				);
			}

			ShaderModelNotificationPtr = FSlateNotificationManager::Get().AddNotification(Info);
		}
	}
#endif // PLATFORM_WINDOWS || PLATFORM_LINUX
}
#endif // #if WITH_EDITOR

void URendererSettings::SanitizeReflectionCaptureResolution()
{
	const int32 MaxCubemapResolution = GetMaxCubeTextureDimension();
	const int32 MinCubemapResolution = 8;

	ReflectionCaptureResolution = FMath::Clamp(int32(FMath::RoundUpToPowerOfTwo(ReflectionCaptureResolution)), MinCubemapResolution, MaxCubemapResolution);

#if WITH_EDITOR
	if (FApp::CanEverRender() && !FApp::IsUnattended())
	{
		SIZE_T TexMemRequired = CalcTextureSize(ReflectionCaptureResolution, ReflectionCaptureResolution, PF_FloatRGBA, FMath::CeilLogTwo(ReflectionCaptureResolution) + 1) * CubeFace_MAX;

		FTextureMemoryStats TextureMemStats;
		RHIGetTextureMemoryStats(TextureMemStats);

		int64 DeviceWorkingMemory = TextureMemStats.GetTotalDeviceWorkingMemory();
		
		if (DeviceWorkingMemory > 0 && TexMemRequired > SIZE_T(DeviceWorkingMemory / 8))
		{
			FNumberFormattingOptions FmtOpts = FNumberFormattingOptions()
				.SetUseGrouping(false)
				.SetMaximumFractionalDigits(2)
				.SetMinimumFractionalDigits(0)
				.SetRoundingMode(HalfFromZero);

			EAppReturnType::Type Response = FPlatformMisc::MessageBoxExt(
				EAppMsgType::YesNo,
				*FText::Format(
					LOCTEXT("MemAllocWarning_Message_ReflectionCubemap", "A resolution of {0} will require {1} of video memory PER reflection capture component. Are you sure?"),
					FText::AsNumber(ReflectionCaptureResolution, &FmtOpts),
					FText::AsMemory(TexMemRequired, &FmtOpts)
				).ToString(),
				*LOCTEXT("MemAllocWarning_Title_ReflectionCubemap", "Memory Allocation Warning").ToString()
			);

			if (Response == EAppReturnType::No)
			{
				ReflectionCaptureResolution = PreEditReflectionCaptureResolution;
			}
		}
		PreEditReflectionCaptureResolution = ReflectionCaptureResolution;
	}
#endif // WITH_EDITOR
}

void URendererSettings::UpdateWorkingColorSpaceAndChromaticities()
{
	using namespace UE::Color;

	if (!FApp::CanEverRenderOrProduceRenderData())
	{
		return;
	}

	switch (WorkingColorSpaceChoice)
	{
	case EWorkingColorSpace::sRGB:
		FColorSpace::SetWorking(FColorSpace(EColorSpace::sRGB));
		break;
	case EWorkingColorSpace::Rec2020:
		FColorSpace::SetWorking(FColorSpace(EColorSpace::Rec2020));
		break;
	case EWorkingColorSpace::ACESAP0:
		FColorSpace::SetWorking(FColorSpace(EColorSpace::ACESAP0));
		break;
	case EWorkingColorSpace::ACESAP1:
		FColorSpace::SetWorking(FColorSpace(EColorSpace::ACESAP1));
		break;
	case EWorkingColorSpace::P3DCI:
		FColorSpace::SetWorking(FColorSpace(EColorSpace::P3DCI));
		break;
	case EWorkingColorSpace::P3D65:
		FColorSpace::SetWorking(FColorSpace(EColorSpace::P3D65));
		break;
	case EWorkingColorSpace::Custom:
		FColorSpace::SetWorking(FColorSpace(RedChromaticityCoordinate, GreenChromaticityCoordinate, BlueChromaticityCoordinate, WhiteChromaticityCoordinate));
		break;
	default:
		check(false);
		break;
	}

	if (WorkingColorSpaceChoice != EWorkingColorSpace::Custom)
	{
		FColorSpace::GetWorking().GetChromaticities(RedChromaticityCoordinate, GreenChromaticityCoordinate, BlueChromaticityCoordinate, WhiteChromaticityCoordinate);
	}
	
	FColorSpace UpdatedColorSpace = FColorSpace::GetWorking();

	ENQUEUE_RENDER_COMMAND(WorkingColorSpaceCommand)(
		[UpdatedColorSpace](FRHICommandList& RHICmdList)
		{
			// Set or update the global uniform buffer for Working Color Space conversions.
			GDefaultWorkingColorSpaceUniformBuffer.Update(RHICmdList, UpdatedColorSpace);
		});
}


URendererOverrideSettings::URendererOverrideSettings(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	SectionName = TEXT("Rendering Overrides");	
}

void URendererOverrideSettings::PostInitProperties()
{
	Super::PostInitProperties();

#if WITH_EDITOR
	if (IsTemplate())
	{
		ImportConsoleVariableValues();
	}
#endif // #if WITH_EDITOR
}

#if WITH_EDITOR
void URendererOverrideSettings::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);	

	if (PropertyChangedEvent.Property)
	{
		ExportValuesToConsoleVariables(PropertyChangedEvent.Property);		
	}
}
#endif // #if WITH_EDITOR

#undef LOCTEXT_NAMESPACE

