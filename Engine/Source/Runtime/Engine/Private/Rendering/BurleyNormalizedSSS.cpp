// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
BurleyNormalizedSSS.cpp: Compute the transmition profile for Burley normalized SSS
=============================================================================*/


#include "Rendering/BurleyNormalizedSSS.h"
#include "Math/Vector.h"
#include "HAL/IConsoleManager.h"
#include "Engine/SubsurfaceScatteringMath.h"

static TAutoConsoleVariableDeprecated_NoReplace<int32> CVarSSProfilesTransmissionUseLegacy(
	TEXT("r.SSProfiles.Transmission.UseLegacy"),
	1,
	TEXT("5.8"),
	EShadowCVarBehavior::Ensure,
	TEXT("r.SSProfiles.Transmission.UseLegacy has been removed. If it was set to 0 and has expected results. Need to scale down `DistanceScale` by 10.")
);

// estimated from the sampling interval, 1/TargetBufferSize(1/32) and MaxTransmissionProfileDistance. If any is changed, this parameter should be re-estimated.
const float ProfileRadiusOffset = 0.06f;

inline FVector Burley_ScatteringProfile(float RadiusInMm, FLinearColor SurfaceAlbedo, FVector ScalingFactor, FLinearColor DiffuseMeanFreePathInMm)
{  
	return SubsurfaceScattering::BurleyNormalized::DiffusionProfile(RadiusInMm, SurfaceAlbedo, ScalingFactor,DiffuseMeanFreePathInMm);
}

inline FLinearColor Burley_TransmissionProfile(float RadiusInMm, FLinearColor SurfaceAlbedo, FVector ScalingFactor, FLinearColor DiffuseMeanFreePathInMm)
{
	FVector Transmission = SubsurfaceScattering::BurleyNormalized::TransmissionProfile(RadiusInMm, SurfaceAlbedo, ScalingFactor,DiffuseMeanFreePathInMm);
	return FLinearColor(Transmission.X, Transmission.Y, Transmission.Z);
}

//--------------------------------------------------------------------------
//Map burley ColorFallOff to Burley SurfaceAlbedo and diffuse mean free path.
void MapFallOffColor2SurfaceAlbedoAndDiffuseMeanFreePath(float FalloffColor, float& SurfaceAlbedo, float& DiffuseMeanFreePath)
{
	//@TODO, use picewise function to separate Falloffcolor to around (0,0.2) and (0.2, 1) to make it more correct
	//map Falloffcolor to SurfaceAlbedo with 4 polynomial, error < 2e-3.
	float X = FalloffColor;
	float X2 = X * X;
	float X4 = X2 * X2;
#if 0
	// max error happens around 0.1, which is -4.8e-3. The others are less than 2.5e-3.
	SurfaceAlbedo = 5.883 * X4 * X2 - 19.88 * X4 * X + 26.08 * X4 - 16.59 * X2 * X + 5.143 * X2 + 0.2636 * X + 0.01098;
	// max error happens around 0.1, which is -3.8e-3.
	DiffuseMeanFreePath = 4.78 * X4 * X2 - 5.178 * X4 * X + 5.2154 * X4 - 4.424 * X2 * X + 1.636 * X2 + 0.4067 * X + 0.006853;
#else
	// max error happens around 0, which is 1e-4. The others are less than 2e-5.
	SurfaceAlbedo = 0.906 * X + 0.00004;
	// max error happens around 0.95, which is -1e-4.
	DiffuseMeanFreePath = 10.39 * X4 * X -15.18 * X4 + 8.332 * X2 * X -2.039 * X2 + 0.7279 * X - 0.0014;
#endif
}

void ComputeMirroredBSSSKernel(FLinearColor* TargetBuffer, uint32 TargetBufferSize,
	FLinearColor SurfaceAlbedo, FLinearColor DiffuseMeanFreePath, float ScatterRadius)
{
	check(TargetBuffer);
	check(TargetBufferSize > 0);

	uint32 nNonMirroredSamples = TargetBufferSize;
	int32 nTotalSamples = nNonMirroredSamples * 2 - 1;

	FVector ScalingFactor = SubsurfaceScattering::BurleyNormalized::GetSearchLightDiffuseScalingFactor(SurfaceAlbedo);
	// we could generate Out directly but the original code form SeparableSSS wasn't done like that so we convert it later
	// .a is in mm
	check(nTotalSamples < 64);

	FLinearColor kernel[64];
	{
		const float Range = (nTotalSamples > 20 ? 3.0f : 2.0f);
		// tweak constant
		const float Exponent = 2.0f;

		// Calculate the offsets:
		float step = 2.0f * Range / (nTotalSamples - 1);
		for (int i = 0; i < nTotalSamples; i++)
		{
			float o = -Range + float(i) * step;
			float sign = o < 0.0f ? -1.0f : 1.0f;
			kernel[i].A = Range * sign * FMath::Abs(FMath::Pow(o, Exponent)) / FMath::Pow(Range, Exponent);
		}
		// Center sample should always be zero, but might not be due to potential roundoff error.
		kernel[nTotalSamples / 2].A = 0.0f;

		//Scale the profile sampling radius. This scale enables the sampling between [-3*SpaceScale,+3*SpaceScale] instead of 
		//the default [-3,3] range when fetching kernel parameters.
		const float SpaceScale = ScatterRadius * 10.0f;// from cm to mm

		// Calculate the weights:
		for (int32 i = 0; i < nTotalSamples; i++)
		{
			float w0 = i > 0 ? FMath::Abs(kernel[i].A - kernel[i - 1].A) : 0.0f;
			float w1 = i < nTotalSamples - 1 ? FMath::Abs(kernel[i].A - kernel[i + 1].A) : 0.0f;
			float area = (w0 + w1) / 2.0f;
			FVector t = area * Burley_ScatteringProfile(FMath::Abs(kernel[i].A)*SpaceScale, SurfaceAlbedo, ScalingFactor,DiffuseMeanFreePath);
			kernel[i].R = t.X;
			kernel[i].G = t.Y;
			kernel[i].B = t.Z;
		}

		// We still need to do a small tweak to get the radius to visually match. Multiplying by 2.0 seems to fix it.
		// Match that in GetSubsurfaceProfileKernel in PostProcessSubsurface.usf
		const float StepScale = 2.0f;
		for (int32 i = 0; i < nTotalSamples; i++)
		{
			kernel[i].A *= StepScale;
		}

		// We want the offset 0.0 to come first:
		FLinearColor t = kernel[nTotalSamples / 2];

		for (int i = nTotalSamples / 2; i > 0; i--)
		{
			kernel[i] = kernel[i - 1];
		}
		kernel[0] = t;

		// Normalize the weights in RGB
		{
			FVector sum = FVector(0, 0, 0);

			for (int i = 0; i < nTotalSamples; i++)
			{
				sum.X += kernel[i].R;
				sum.Y += kernel[i].G;
				sum.Z += kernel[i].B;
			}

			for (int i = 0; i < nTotalSamples; i++)
			{
				kernel[i].R /= sum.X;
				kernel[i].G /= sum.Y;
				kernel[i].B /= sum.Z;
			}
		}

		/* we do that in the shader for better quality with half res

		// Tweak them using the desired strength. The first one is:
		//     lerp(1.0, kernel[0].rgb, strength)
		kernel[0].R = FMath::Lerp(1.0f, kernel[0].R, SubsurfaceColor.R);
		kernel[0].G = FMath::Lerp(1.0f, kernel[0].G, SubsurfaceColor.G);
		kernel[0].B = FMath::Lerp(1.0f, kernel[0].B, SubsurfaceColor.B);

		for (int i = 1; i < nTotalSamples; i++)
		{
			kernel[i].R *= SubsurfaceColor.R;
			kernel[i].G *= SubsurfaceColor.G;
			kernel[i].B *= SubsurfaceColor.B;
		}*/
	}

	// generate output (remove negative samples)
	{
		// center sample
		TargetBuffer[0] = kernel[0];

		// all positive samples
		for (uint32 i = 0; i < nNonMirroredSamples - 1; i++)
		{
			TargetBuffer[i + 1] = kernel[nNonMirroredSamples + i];
		}
	}
}



void ComputeTransmissionProfileBurley(FLinearColor* TargetBuffer, uint32 TargetBufferSize, 
									FLinearColor FalloffColor, float ExtinctionScale,
									FLinearColor SurfaceAlbedo, FLinearColor DiffuseMeanFreePathInMm,
									float WorldUnitScale, FLinearColor TransmissionTintColor)
{
	check(TargetBuffer);
	check(TargetBufferSize > 0);

	// Unit scale should be independent to the base unit.
	// Example of scaling
	// ----------------------------------------
	// DistanceCm * UnitScale * CmToMm = Value (mm)
	// ----------------------------------------
	//   1          0.1         10     =   1mm
	//   1          1.0         10     =  10mm
	//   1         10.0         10     = 100mm

	const float SubsurfaceScatteringUnitInCm = 0.1f;
	const float UnitScale = WorldUnitScale / SubsurfaceScatteringUnitInCm;
	float InvUnitScale = 1.0 / UnitScale; // Scaling the unit is equivalent to inverse scaling of the profile.

	static float MaxTransmissionProfileDistance = 5.0f; // See SSSS_MAX_TRANSMISSION_PROFILE_DISTANCE in TransmissionCommon.ush
	static float CmToMm = 10.0f;
	//assuming that the volume albedo is the same to the surface albedo for transmission.
	FVector ScalingFactor = SubsurfaceScattering::BurleyNormalized::GetSearchLightDiffuseScalingFactor(SurfaceAlbedo);

	const float InvSize = 1.0f / TargetBufferSize;

	for (uint32 i = 0; i < TargetBufferSize; ++i)
	{
		const float DistanceInMm = i * InvSize * (MaxTransmissionProfileDistance * CmToMm) * InvUnitScale;
		const float OffsetInMm = (ProfileRadiusOffset * CmToMm) * InvUnitScale;

		// The profile does not contain surface albedo, the SSS recombine pass will use stored base color as it.
		FLinearColor TransmissionProfile = Burley_TransmissionProfile(DistanceInMm + OffsetInMm, FLinearColor::White, ScalingFactor, DiffuseMeanFreePathInMm);
		TargetBuffer[i] = TransmissionProfile * TransmissionTintColor; // Apply tint to the profile
		//Use Luminance of scattering as SSSS shadow.
		TargetBuffer[i].A = exp(-DistanceInMm * ExtinctionScale);
	}

	// Do this is because 5mm is not enough cool down the scattering to zero, although which is small number but after tone mapping still noticeable
	// so just Let last pixel be 0 which make sure thickness great than MaxRadius have no scattering
	static bool bMakeLastPixelBlack = true;
	if (bMakeLastPixelBlack)
	{
		TargetBuffer[TargetBufferSize - 1] = FLinearColor::Black;
	}
}
