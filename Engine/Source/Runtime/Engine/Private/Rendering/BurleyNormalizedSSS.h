// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
BurleyNormalizedSSS.h: Compute the transmission profile and convert parameters
=============================================================================*/

#pragma once

#include "CoreMinimal.h"

// @param TargetBuffer, needs to be preallocated with TargetBufferSize (RGB is the sample weight, A is the offset), [0] is the center samples, following elements need to be mirrored with A, -A
// @param TargetBufferSize >0
// @parma FalloffColor see SubsurfaceProfile.h
// @parma SurfaceAlbedo see SubsurfaceProfile.h
// @parma DiffuseMeanFreePath mean free path color scaled by mean free path distance (in mm).
// @param WorldUnitScale see SubsurfaceProfile.h
// @param TransmissionTintColor see SubsurfaceProfile.h
void ComputeTransmissionProfileBurley(FLinearColor* TargetBuffer, uint32 TargetBufferSize, 
									FLinearColor FalloffColor, float ExtinctionScale,
									FLinearColor SurfaceAlbedo, FLinearColor DiffuseMeanFreePath, 
									float WorldUnitScale,FLinearColor TransmissionTintColor);

// Compute the Separable Kernel using Burley profile
void ComputeMirroredBSSSKernel(FLinearColor* TargetBuffer, uint32 TargetBufferSize, FLinearColor SurfaceAlbedo,
	FLinearColor DiffuseMeanFreePath, float ScatterRadius);

//@param FalloffColor from Separable SSS
//@param SurfaceAlbedo mapped from FallofColor 
//@param DiffuseMeanFreePath mapped from FallofColor
void MapFallOffColor2SurfaceAlbedoAndDiffuseMeanFreePath(float FalloffColor, float& SurfaceAlbedo, float& DiffuseMeanFreePath);
