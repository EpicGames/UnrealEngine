// Copyright Epic Games, Inc. All Rights Reserved.

#include "Rendering/ColorVertexBuffer.h"
#include "Components.h"
#include "EngineLogs.h"
#include "EngineUtils.h"
#include "ProfilingDebugging/LoadTimeTracker.h"
#include "StaticMeshVertexData.h"
#include "VertexFactory.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "GlobalRenderResources.h"
#include "RHICommandList.h"
#include "RHIResourceReplace.h"

/*-----------------------------------------------------------------------------
FColorVertexBuffer
-----------------------------------------------------------------------------*/

/** The implementation of the static mesh color-only vertex data storage type. */
class FColorVertexData :
	public TStaticMeshVertexData<FColor>
{
public:
	FColorVertexData(bool InNeedsCPUAccess = false)
		: TStaticMeshVertexData<FColor>(InNeedsCPUAccess)
	{
	}
};


FColorVertexBuffer::FColorVertexBuffer():
	VertexData(NULL),
	Data(NULL),
	Stride(0),
	NumVertices(0)
{
}

FColorVertexBuffer::FColorVertexBuffer( const FColorVertexBuffer &rhs ):
	VertexData(NULL),
	Data(NULL),
	Stride(0),
	NumVertices(0)
{
	if (rhs.VertexData)
	{
		InitFromColorArray(reinterpret_cast<FColor*>(rhs.VertexData->GetDataPointer()), rhs.VertexData->Num());
	}
	else
	{
		CleanUp();
	}
}

FColorVertexBuffer::~FColorVertexBuffer()
{
	CleanUp();
}

/** Delete existing resources */
void FColorVertexBuffer::CleanUp()
{
	if (VertexData)
	{
		delete VertexData;
		VertexData = NULL;
	}
}

void FColorVertexBuffer::Init(uint32 InNumVertices, bool bNeedsCPUAccess)
{
	NumVertices = InNumVertices;
	NeedsCPUAccess = bNeedsCPUAccess;

	// Allocate the vertex data storage type.
	AllocateData(bNeedsCPUAccess);

	// Allocate the vertex data buffer.
	VertexData->ResizeBuffer(NumVertices);
	Data = InNumVertices ? VertexData->GetDataPointer() : nullptr;
}

/**
* Initializes the buffer with the given vertices, used to convert legacy layouts.
* @param InVertices - The vertices to initialize the buffer with.
*/
void FColorVertexBuffer::Init(const TArray<FStaticMeshBuildVertex>& InVertices, bool bNeedsCPUAccess)
{
	const FConstMeshBuildVertexView VertexView = MakeConstMeshBuildVertexView(InVertices);
	Init(VertexView, bNeedsCPUAccess);
}

void FColorVertexBuffer::Init(const FConstMeshBuildVertexView& InVertices, bool bNeedsCPUAccess)
{
	NeedsCPUAccess = bNeedsCPUAccess;

	const int32 ColorCount = InVertices.Color.Num();

	if (ColorCount == 0)
	{
		// Ensure no vertex data is allocated.
		CleanUp();

		// Clear the vertex count and stride.
		Stride = 0;
		NumVertices = 0;
	}
	else
	{
		Init(ColorCount, bNeedsCPUAccess);

		// Copy the vertex colors into the buffer.
		for (int32 VertexIndex = 0;VertexIndex < ColorCount; ++VertexIndex)
		{
			VertexColor(VertexIndex) = InVertices.Color[VertexIndex];
		}
	}
}

/**
* Initializes this vertex buffer with the contents of the given vertex buffer.
* @param InVertexBuffer - The vertex buffer to initialize from.
*/
void FColorVertexBuffer::Init(const FColorVertexBuffer& InVertexBuffer, bool bNeedsCPUAccess)
{
	NeedsCPUAccess = bNeedsCPUAccess;
	if ( NumVertices )
	{
		Init(InVertexBuffer.GetNumVertices(), bNeedsCPUAccess);
		const uint8* InData = InVertexBuffer.Data;
		FMemory::Memcpy( Data, InData, Stride * NumVertices );
	}
}

bool FColorVertexBuffer::AppendVertices( const FStaticMeshBuildVertex* Vertices, const uint32 NumVerticesToAppend )
{
	const uint64 TotalNumVertices = (uint64)NumVertices + (uint64)NumVerticesToAppend;
	if (!ensureMsgf(TotalNumVertices < INT32_MAX, TEXT("FColorVertexBuffer::AppendVertices adding %u to %u vertices exceeds INT32_MAX limit"), NumVerticesToAppend, NumVertices))
	{
		return false;
	}

	if (VertexData == nullptr && NumVerticesToAppend > 0)
	{
		check( NumVertices == 0 );

		// Allocate the vertex data storage type if the buffer was never allocated before
		AllocateData(NeedsCPUAccess);
	}

	if( NumVerticesToAppend > 0 )
	{
		// @todo: check if all opaque white, and if so append nothing

		check( VertexData != nullptr );	// Must only be called after Init() has already initialized the buffer!
		check( Vertices != nullptr );

		const uint32 FirstDestVertexIndex = NumVertices;
		NumVertices += NumVerticesToAppend;
		VertexData->ResizeBuffer( NumVertices );
		if( NumVertices > 0 )
		{
			Data = VertexData->GetDataPointer();

			// Copy the vertices into the buffer.
			for( uint32 VertexIter = 0; VertexIter < NumVerticesToAppend; ++VertexIter )
			{
				const FStaticMeshBuildVertex& SourceVertex = Vertices[ VertexIter ];

				const uint32 DestVertexIndex = FirstDestVertexIndex + VertexIter;
				VertexColor( DestVertexIndex ) = SourceVertex.Color;
			}
		}
	}

	return true;
}

/**
 * Serializer
 *
 * @param	Ar				Archive to serialize with
 * @param	bNeedsCPUAccess	Whether the elements need to be accessed by the CPU
 */
void FColorVertexBuffer::Serialize( FArchive& Ar, bool bNeedsCPUAccess )
{
	NeedsCPUAccess = bNeedsCPUAccess;
	FStripDataFlags StripFlags(Ar, 0, FPackageFileVersion::CreateUE4Version(VER_UE4_STATIC_SKELETAL_MESH_SERIALIZATION_FIX));

	if (Ar.IsSaving() && NumVertices > 0 && VertexData == NULL)
	{
		// ...serialize as if the vertex count were zero. Else on load serialization breaks.
		// This situation should never occur because VertexData should not be null if NumVertices
		// is greater than zero. So really this should be a checkf but I don't want to crash
		// the Editor when saving a package.
		UE_LOGF(LogStaticMesh, Warning, "Color vertex buffer being saved with NumVertices=%d Stride=%d VertexData=NULL. This should never happen.",
			NumVertices, Stride );

		int32 SerializedStride = 0;
		int32 SerializedNumVertices = 0;
		Ar << SerializedStride << SerializedNumVertices;
	}
	else
	{
		SerializeMetaData(Ar);

		if (Ar.IsLoading() && NumVertices > 0)
		{
			// Allocate the vertex data storage type.
			AllocateData(bNeedsCPUAccess);
		}

		if (!StripFlags.IsAudioVisualDataStripped() || Ar.IsCountingMemory())
		{
			if (VertexData != nullptr)
			{
				// Serialize the vertex data.
				VertexData->Serialize(Ar);
			}
		}

		if (Ar.IsLoading())
		{
			if (!StripFlags.IsAudioVisualDataStripped())
			{
				if (VertexData != nullptr && VertexData->Num() > 0)
				{
					// Make a copy of the vertex data pointer.
					Data = VertexData->GetDataPointer();
				}
			}
			else
			{
				// if we stripped all the other stuff and decided not to serialize it in probably need to strip the NumVertices Too
				NumVertices = Stride = 0;
			}
		}
	}
}

void FColorVertexBuffer::SerializeMetaData(FArchive& Ar)
{
	Ar << Stride << NumVertices;
}

void FColorVertexBuffer::SetMetaData(uint32 InStride, uint32 InNumVertices)
{
	Stride = InStride;
	NumVertices = InNumVertices;
}

void FColorVertexBuffer::ClearMetaData()
{
	Stride = NumVertices = 0;
}


/** Export the data to a string, used for editor Copy&Paste. */
void FColorVertexBuffer::ExportText(FString &ValueStr) const
{
	// the following code only works if there is data and this method should only be called if there is data
	check(NumVertices);

	ValueStr += FString::Printf(TEXT("ColorVertexData(%i)=("), NumVertices);

	// 9 characters per color (ARGB in hex plus comma)
	ValueStr.Reserve(ValueStr.Len() + NumVertices * 9);

	for(uint32 i = 0; i < NumVertices; ++i)
	{
		uint32 Raw = VertexColor(i).DWColor();
		// does not handle endianess	
		// order: ARGB
		TCHAR ColorString[10];

		// does not use FString::Printf for performance reasons
		FCString::Sprintf(ColorString, TEXT("%.8x,"), Raw);
		ValueStr += ColorString;
	}

	// replace , by )
	ValueStr[ValueStr.Len() - 1] = ')';
}



/** Export the data from a string, used for editor Copy&Paste. */
void FColorVertexBuffer::ImportText(const TCHAR* SourceText)
{
	check(SourceText);
	check(!VertexData);

	uint32 VertexCount;

	if (FParse::Value(SourceText, TEXT("ColorVertexData("), VertexCount))
	{
		while(*SourceText && *SourceText != TEXT(')'))
		{
			++SourceText;
		}

		while(*SourceText && *SourceText != TEXT('('))
		{
			++SourceText;
		}

		check(*SourceText == TEXT('('));
		++SourceText;

		NumVertices = VertexCount;
		AllocateData(NeedsCPUAccess);
		VertexData->ResizeBuffer(NumVertices);
		uint8 *Dst = (uint8 *)VertexData->GetDataPointer();

		// 9 characters per color (ARGB in hex plus comma)
		for( uint32 i = 0; i < NumVertices; ++i)
		{
			// does not handle endianess or malformed input
			*Dst++ = FParse::HexDigit(SourceText[6]) * 16 + FParse::HexDigit(SourceText[7]);
			*Dst++ = FParse::HexDigit(SourceText[4]) * 16 + FParse::HexDigit(SourceText[5]);
			*Dst++ = FParse::HexDigit(SourceText[2]) * 16 + FParse::HexDigit(SourceText[3]);
			*Dst++ = FParse::HexDigit(SourceText[0]) * 16 + FParse::HexDigit(SourceText[1]);
			SourceText += 9;
		}
		check(*(SourceText - 1) == TCHAR(')'));

		// Make a copy of the vertex data pointer.
		Data = VertexData->GetDataPointer();

		BeginInitResource(this);
	}
}

/**
* Specialized assignment operator, only used when importing LOD's.  
*/
void FColorVertexBuffer::operator=(const FColorVertexBuffer &Other)
{
	//VertexData doesn't need to be allocated here because Build will be called next,
	delete VertexData;
	VertexData = NULL;
}

void FColorVertexBuffer::GetVertexColors( TArray<FColor>& OutColors ) const
{
	if( VertexData != NULL && NumVertices > 0 )
	{
		OutColors.SetNumUninitialized( NumVertices );

		FMemory::Memcpy( OutColors.GetData(), VertexData->GetDataPointer(), NumVertices * VertexData->GetStride() ) ;
	}
}

/** Load from raw color array */
void FColorVertexBuffer::InitFromColorArray( const FColor *InColors, const uint32 Count, const uint32 InStride, bool bNeedsCPUAccess)
{
	check( Count > 0 );

	NumVertices = Count;
	NeedsCPUAccess = bNeedsCPUAccess;

	// Allocate the vertex data storage type.
	AllocateData(bNeedsCPUAccess);

	// Copy the colors
	{
		VertexData->ResizeBuffer(Count);

		const uint8 *Src = (const uint8 *)InColors;
		FColor *Dst = (FColor *)VertexData->GetDataPointer();

		for( uint32 i = 0; i < Count; ++i)
		{
			*Dst++ = *(const FColor*)Src;

			Src += InStride;
		}
	}

	// Make a copy of the vertex data pointer.
	Data = VertexData->GetDataPointer();
}

bool FColorVertexBuffer::GetAllowCPUAccess() const
{
	return VertexData ? VertexData->GetAllowCPUAccess() : false;
}

uint32 FColorVertexBuffer::GetAllocatedSize() const
{
	if(VertexData)
	{
		return VertexData->GetResourceSize();
	}
	else
	{
		return 0;
	}
}

FBufferRHIRef FColorVertexBuffer::CreateRHIBuffer(FRHICommandListBase& RHICmdList)
{
	return FRenderResource::CreateRHIBuffer(RHICmdList, VertexData, NumVertices, BUF_Static | BUF_ShaderResource, TEXT("FColorVertexBuffer"));
}

void FColorVertexBuffer::InitRHIForStreaming(FRHIBuffer* IntermediateBuffer, FRHIResourceReplaceBatcher& Batcher)
{
	if (VertexBufferRHI && IntermediateBuffer)
	{
		Batcher.EnqueueReplace(VertexBufferRHI, IntermediateBuffer);
	}
}

void FColorVertexBuffer::ReleaseRHIForStreaming(FRHIResourceReplaceBatcher& Batcher)
{
	if (VertexBufferRHI)
	{
		Batcher.EnqueueReplace(VertexBufferRHI, nullptr);
	}
}

void FColorVertexBuffer::InitRHI(FRHICommandListBase& RHICmdList)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FColorVertexBuffer::InitRHI);
	SCOPED_LOADTIMER(FColorVertexBuffer_InitRHI);

	VertexBufferRHI = CreateRHIBuffer(RHICmdList);
	if (VertexBufferRHI && RHISupportsManualVertexFetch(GMaxRHIShaderPlatform))
	{
		ColorComponentsSRV = RHICmdList.CreateShaderResourceView(
			VertexBufferRHI,
			FRHIViewDesc::CreateBufferSRV()
				.SetType(FRHIViewDesc::EBufferType::Typed)
				.SetFormat(PF_R8G8B8A8));
	}
}

void FColorVertexBuffer::ReleaseRHI()
{
	ColorComponentsSRV.SafeRelease();
	FVertexBuffer::ReleaseRHI();
}

void FColorVertexBuffer::AllocateData( bool bNeedsCPUAccess /*= true*/ )
{
	// Clear any old VertexData before allocating.
	CleanUp();

	VertexData = new FColorVertexData(bNeedsCPUAccess);
	// Calculate the vertex stride.
	Stride = VertexData->GetStride();
}

void FColorVertexBuffer::BindColorVertexBuffer(const FVertexFactory* VertexFactory, FStaticMeshDataType& StaticMeshData) const
{
	if (GetNumVertices() == 0)
	{
		BindDefaultColorVertexBuffer(VertexFactory, StaticMeshData, NullBindStride::ZeroForDefaultBufferBind);
		return;
	}

	StaticMeshData.ColorComponentsSRV = ColorComponentsSRV;
	StaticMeshData.ColorIndexMask = ~0u;

	{	
		StaticMeshData.ColorComponent = FVertexStreamComponent(
			this,
			0,	// Struct offset to color
			GetStride(),
			VET_Color,
			EVertexStreamUsage::ManualFetch
		);
	}
}

void FColorVertexBuffer::BindDefaultColorVertexBuffer(const FVertexFactory* VertexFactory, FStaticMeshDataType& StaticMeshData, NullBindStride BindStride)
{
	StaticMeshData.ColorComponentsSRV = GNullColorVertexBuffer.VertexBufferSRV;
	StaticMeshData.ColorIndexMask = 0;

	{
		const bool bBindForDrawOverride = BindStride == NullBindStride::FColorSizeForComponentOverride;

		StaticMeshData.ColorComponent = FVertexStreamComponent(
			&GNullColorVertexBuffer,
			0, // Struct offset to color
			bBindForDrawOverride ? sizeof(FColor) : 0, //asserted elsewhere
			VET_Color,
			bBindForDrawOverride ? (EVertexStreamUsage::ManualFetch | EVertexStreamUsage::Overridden) : (EVertexStreamUsage::ManualFetch)
		);
	}
}
