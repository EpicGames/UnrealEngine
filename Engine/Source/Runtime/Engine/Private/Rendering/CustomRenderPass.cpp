// Copyright Epic Games, Inc. All Rights Reserved.

#include "Rendering/CustomRenderPass.h"
#include "IRenderCaptureProvider.h"
#include "RenderGraphBuilder.h"

FCustomRenderPassBase::FCustomRenderPassBase(const FString& InDebugName, ERenderMode InRenderMode, ERenderOutput InRenderOutput, const FIntPoint& InRenderTargetSize)
	: DebugName(InDebugName)
	, RenderMode(InRenderMode)
	, RenderOutput(InRenderOutput)
	, RenderTargetSize(InRenderTargetSize)
{}

void FCustomRenderPassBase::BeginPass(FRDGBuilder& GraphBuilder)
{
	// Optionally, perform a render capture when this pass runs :
	if ((RenderCaptureType == ERenderCaptureType::Capture) || (RenderCaptureType == ERenderCaptureType::BeginCapture))
	{
		GraphBuilder.AddPass(
			RDG_EVENT_NAME("BeginCapture"),
			ERDGPassFlags::NeverCull,
			[FileName = RenderCaptureFileName](FRHICommandListImmediate& RHICommandListLocal)
		{
			IRenderCaptureProvider::Get().BeginCapture(&RHICommandListLocal, IRenderCaptureProvider::ECaptureFlags_Launch, FileName);
		});
	}

	OnBeginPass(GraphBuilder);
}

void FCustomRenderPassBase::PreRender(FRDGBuilder& GraphBuilder)
{
	OnPreRender(GraphBuilder);
}

void FCustomRenderPassBase::PostRender(FRDGBuilder& GraphBuilder)
{
	OnPostRender(GraphBuilder);
}

void FCustomRenderPassBase::EndPass(FRDGBuilder & GraphBuilder)
{
	OnEndPass(GraphBuilder);

	// End the optional render capture after this pass has run :
	if ((RenderCaptureType == ERenderCaptureType::Capture) || (RenderCaptureType == ERenderCaptureType::EndCapture))
	{
		GraphBuilder.AddPass(
			RDG_EVENT_NAME("EndCapture"),
			ERDGPassFlags::NeverCull,
			[](FRHICommandListImmediate& RHICommandListLocal)
		{
			IRenderCaptureProvider::Get().EndCapture(&RHICommandListLocal);
		});
	}
}

ESceneCaptureSource FCustomRenderPassBase::GetSceneCaptureSource() const
{
	if (RenderOutput == ERenderOutput::SceneDepth)
		return SCS_SceneDepth;
	else if (RenderOutput == ERenderOutput::DeviceDepth)
		return SCS_DeviceDepth;
	else if (RenderOutput == ERenderOutput::SceneColorAndDepth)
		return SCS_SceneColorSceneDepth;
	else if (RenderOutput == ERenderOutput::SceneColorAndAlpha)
		return SCS_SceneColorHDR;
	else if (RenderOutput == ERenderOutput::SceneColorNoAlpha)
		return SCS_SceneColorHDRNoAlpha;
	else if (RenderOutput == ERenderOutput::BaseColor)
		return SCS_BaseColor;
	else if (RenderOutput == ERenderOutput::Normal)
		return SCS_Normal;
	else
		return SCS_MAX;
}

void FCustomRenderPassBase::PerformRenderCapture(ERenderCaptureType InRenderCaptureType, const FString& InFileName)
{
	if (!IRenderCaptureProvider::IsAvailable() || (InRenderCaptureType == ERenderCaptureType::NoCapture))
	{
		return;
	}

	RenderCaptureType = InRenderCaptureType;
	RenderCaptureFileName = InFileName;
}

void FCustomRenderPassBase::SetUserData(TUniquePtr<ICustomRenderPassUserData>&& InUserData)
{
	UserDatas.Add(InUserData->GetTypeName(), MoveTemp(InUserData));
}

ICustomRenderPassUserData* FCustomRenderPassBase::GetUserData(const FName& InTypeName) const
{
	const TUniquePtr<ICustomRenderPassUserData>* Found = UserDatas.Find(InTypeName);
	return (Found != nullptr) ? Found->Get() : nullptr;
}
