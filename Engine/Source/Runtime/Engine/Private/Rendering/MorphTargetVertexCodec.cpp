// Copyright Epic Games, Inc. All Rights Reserved.

#include "Rendering/MorphTargetVertexCodec.h"

#include "Animation/MorphTarget.h"
#include "Containers/Array.h"
#include "Math/Vector.h"

#include "Engine/EngineConsoleCommandExecutor.h"

namespace UE::MorphTargetVertexCodec
{

constexpr uint32 IndexMaxBits = 31u;
constexpr uint32 PositionMaxBits = 28u;				// Probably more than we need, but let's just allow it to go this high to be safe for now.
// For larger deltas this can even be more precision than what was in the float input data!
// Maybe consider float-like or exponential encoding of large values?
constexpr float PositionMinValue = -134217728.0f;	// -2^(MaxBits-1)
constexpr float PositionMaxValue = 134217720.0f;	// Largest float smaller than 2^(MaxBits-1)-1
// Using 134217727.0f would NOT work as it would be rounded up to 134217728.0f, which is
// outside the range.

constexpr uint32 TangentZMaxBits = 16u;
constexpr float TangentZMinValue = -32768.0f;		// -2^(MaxBits-1)
constexpr float TangentZMaxValue = 32767.0f;		//  2^(MaxBits-1)-1


class FDwordBitWriter
{
public:
	FDwordBitWriter(TArray<uint32>& Buffer) :
		Buffer(Buffer),
		PendingBits(0ull),
		NumPendingBits(0)
	{
	}

	void PutBits(uint32 Bits, uint32 NumBits)
	{
		check(static_cast<uint64>(Bits) < (1ull << NumBits));
		PendingBits |= static_cast<uint64>(Bits) << NumPendingBits;
		NumPendingBits += NumBits;

		while (NumPendingBits >= 32)
		{
			Buffer.Add(static_cast<uint32>(PendingBits));
			PendingBits >>= 32;
			NumPendingBits -= 32;
		}
	}

	void Flush()
	{
		if (NumPendingBits > 0)
			Buffer.Add(static_cast<uint32>(PendingBits));
		PendingBits = 0;
		NumPendingBits = 0;
	}

private:
	TArray<uint32>& Buffer;
	uint64 			PendingBits;
	int32 			NumPendingBits;
};

class FDwordViewBitWriter
{
public:
	FDwordViewBitWriter(TArrayView<uint32> Buffer) :
		Buffer(Buffer),
		PendingBits(0ull),
		NumPendingBits(0),
		NumElements(0)
	{
	}

	void PutBits(uint32 Bits, uint32 NumBits)
	{
		check(static_cast<uint64>(Bits) < (1ull << NumBits));
		PendingBits |= static_cast<uint64>(Bits) << NumPendingBits;
		NumPendingBits += NumBits;

		while (NumPendingBits >= 32)
		{
			Buffer[NumElements++] = static_cast<uint32>(PendingBits);
			PendingBits >>= 32;
			NumPendingBits -= 32;
		}
	}

	void Flush()
	{
		if (NumPendingBits > 0)
		{
			Buffer[NumElements++] = (static_cast<uint32>(PendingBits));
		}
		PendingBits = 0;
		NumPendingBits = 0;
	}

private:
	TArrayView<uint32> Buffer;
	uint64 			PendingBits;
	int32 			NumPendingBits;
	int32           NumElements;
};


class FDwordBitReader
{
public:
	FDwordBitReader(TConstArrayView<uint32> InBuffer, uint32 InOffset = 0) :
		Buffer(InBuffer),
		Offset(InOffset)
	{
	}

	uint32 GetBits(uint32 NumBits)
	{
		check(NumBits <= 32);
		check(Offset + NumBits <= (Buffer.Num() * 32u));
		
		if (NumBits == 0)
		{
			return 0;
		}
		
		const uint32 BaseIndex = Offset >> 5u;
		const uint32 BitOffset = Offset & 31u;

		// Advance
		Offset += NumBits;

		if ((BitOffset + NumBits) > 32)
		{
			const uint32 BitMaskLow = ((1 << (32 - BitOffset)) - 1);
			const uint32 BitMaskHigh = (1 << (NumBits + BitOffset - 32)) - 1;
			const uint32 BitOffsetLow = BitOffset;
			const uint32 BitOffsetHigh = 32 - BitOffset;

			const uint32 Low = ((Buffer[BaseIndex + 0] >> BitOffsetLow) & BitMaskLow);
			const uint32 High = ((Buffer[BaseIndex + 1] & BitMaskHigh) << BitOffsetHigh);

			return Low | High;
		}
		else
		{
			const uint32 BitMask = (1ull << NumBits) - 1;
			return (Buffer[BaseIndex] >> BitOffset) & BitMask;
		}
	}

	uint32 GetOffset() const
	{
		return Offset;
	}

private:
	TConstArrayView<uint32> Buffer;
	uint32 Offset = 0;
};

void Encode(
	TConstArrayView<FMorphTargetDelta> InMorphDeltas,
	const TBitArray<>* InVertexNeedsTangents,
	const float InPositionPrecision,
	const float InTangentZPrecision,
	TArray<FDeltaBatchHeader>& OutBatchHeaders,
	TArray<uint32>& OutCompressedVertices
	)
{
	// Simple Morph compression 0.1
	// Instead of storing vertex deltas individually they are organized into batches of 64.
	// Each batch has a header that describes how many bits are allocated to each of the vertex components.
	// Batches also store an explicit offset to its associated data. This makes it trivial to decode batches
	// in parallel, and because deltas are fixed-width inside a batch, deltas can also be decoded in parallel.
	// The result is a semi-adaptive encoding that functions as a crude substitute for entropy coding, that is
	// fast to decode on parallel hardware.

	// Quantization still happens globally to avoid issues with cracks at duplicate vertices.
	// The quantization is artist controlled on a per LOD basis. Higher error tolerance results in smaller deltas
	// and a smaller compressed size.

	const float RcpPositionPrecision = 1.0f / InPositionPrecision;
	const float RcpTangentZPrecision = 1.0f / InTangentZPrecision;

	TArray<FQuantizedDelta> QuantizedDeltas;
	QuantizedDeltas.Reserve(InMorphDeltas.Num());

	bool bVertexIndicesSorted = true;

	int32 PrevVertexIndex = -1;
	for (int32 DeltaIndex = 0; DeltaIndex < InMorphDeltas.Num(); DeltaIndex++)
	{
		const FMorphTargetDelta& MorphDelta = InMorphDeltas[DeltaIndex];
		const FVector3f TangentZDelta = !InVertexNeedsTangents || (InVertexNeedsTangents->IsValidIndex(MorphDelta.SourceIdx) && (*InVertexNeedsTangents)[MorphDelta.SourceIdx]) ? MorphDelta.TangentZDelta : FVector3f::ZeroVector;

		// Check if input is sorted. It usually is, but it might not be.
		if (static_cast<int32>(MorphDelta.SourceIdx) < PrevVertexIndex)
			bVertexIndicesSorted = false;
		PrevVertexIndex = static_cast<int32>(MorphDelta.SourceIdx);

		// Quantize delta
		FQuantizedDelta QuantizedDelta;
		const FVector3f& PositionDelta = MorphDelta.PositionDelta;
		QuantizedDelta.Position.X = FMath::RoundToInt(FMath::Clamp(PositionDelta.X * RcpPositionPrecision, PositionMinValue, PositionMaxValue));
		QuantizedDelta.Position.Y = FMath::RoundToInt(FMath::Clamp(PositionDelta.Y * RcpPositionPrecision, PositionMinValue, PositionMaxValue));
		QuantizedDelta.Position.Z = FMath::RoundToInt(FMath::Clamp(PositionDelta.Z * RcpPositionPrecision, PositionMinValue, PositionMaxValue));
		QuantizedDelta.TangentZ.X = FMath::RoundToInt(FMath::Clamp(TangentZDelta.X * RcpTangentZPrecision, TangentZMinValue, TangentZMaxValue));
		QuantizedDelta.TangentZ.Y = FMath::RoundToInt(FMath::Clamp(TangentZDelta.Y * RcpTangentZPrecision, TangentZMinValue, TangentZMaxValue));
		QuantizedDelta.TangentZ.Z = FMath::RoundToInt(FMath::Clamp(TangentZDelta.Z * RcpTangentZPrecision, TangentZMinValue, TangentZMaxValue));
		QuantizedDelta.Index = MorphDelta.SourceIdx;

		if (QuantizedDelta.Position != FIntVector::ZeroValue || QuantizedDelta.TangentZ != FIntVector::ZeroValue)
		{
			// Only add delta if it is non-zero
			QuantizedDeltas.Add(QuantizedDelta);
		}
	}

	// Sort deltas if the source wasn't already sorted
	if (!bVertexIndicesSorted)
	{
		Algo::Sort(QuantizedDeltas, [](const FQuantizedDelta& A, const FQuantizedDelta& B) { return A.Index < B.Index; });
	}

	// Encode batch deltas
	const uint32 MorphNumBatches = (QuantizedDeltas.Num() + BatchSize - 1u) / BatchSize;
	for (uint32 BatchIndex = 0; BatchIndex < MorphNumBatches; BatchIndex++)
	{
		const uint32 BatchFirstElementIndex = BatchIndex * BatchSize;
		const uint32 NumElements = FMath::Min(BatchSize, QuantizedDeltas.Num() - BatchFirstElementIndex);

		// Calculate batch min/max bounds
		uint32 IndexMin = MAX_uint32;
		uint32 IndexMax = MIN_uint32;
		FIntVector PositionMin = FIntVector(MAX_int32);
		FIntVector PositionMax = FIntVector(MIN_int32);
		FIntVector TangentZMin = FIntVector(MAX_int32);
		FIntVector TangentZMax = FIntVector(MIN_int32);

		for (uint32 LocalElementIndex = 0; LocalElementIndex < NumElements; LocalElementIndex++)
		{
			const FQuantizedDelta& Delta = QuantizedDeltas[BatchFirstElementIndex + LocalElementIndex];

			// Trick: Deltas are sorted by index, so the index increase by at least one per delta.
			//		  Naively this would mean that a batch always spans at least 64 index values and
			//		  indices would have to use at least 6 bits per index.
			//		  If instead of storing the raw index, we store the index relative to its position in the batch,
			//		  then the spanned range becomes 63 smaller.
			//		  For a consecutive range this even gets us down to 0 bits per index!
			check(Delta.Index >= LocalElementIndex);
			const uint32 AdjustedIndex = Delta.Index - LocalElementIndex;
			IndexMin = FMath::Min(IndexMin, AdjustedIndex);
			IndexMax = FMath::Max(IndexMax, AdjustedIndex);

			PositionMin.X = FMath::Min(PositionMin.X, Delta.Position.X);
			PositionMin.Y = FMath::Min(PositionMin.Y, Delta.Position.Y);
			PositionMin.Z = FMath::Min(PositionMin.Z, Delta.Position.Z);

			PositionMax.X = FMath::Max(PositionMax.X, Delta.Position.X);
			PositionMax.Y = FMath::Max(PositionMax.Y, Delta.Position.Y);
			PositionMax.Z = FMath::Max(PositionMax.Z, Delta.Position.Z);

			TangentZMin.X = FMath::Min(TangentZMin.X, Delta.TangentZ.X);
			TangentZMin.Y = FMath::Min(TangentZMin.Y, Delta.TangentZ.Y);
			TangentZMin.Z = FMath::Min(TangentZMin.Z, Delta.TangentZ.Z);

			TangentZMax.X = FMath::Max(TangentZMax.X, Delta.TangentZ.X);
			TangentZMax.Y = FMath::Max(TangentZMax.Y, Delta.TangentZ.Y);
			TangentZMax.Z = FMath::Max(TangentZMax.Z, Delta.TangentZ.Z);
		}

		const uint32 IndexDelta = IndexMax - IndexMin;
		const FIntVector PositionDelta = PositionMax - PositionMin;
		const FIntVector TangentZDelta = TangentZMax - TangentZMin;
		const bool bBatchHasTangents = TangentZMin != FIntVector::ZeroValue || TangentZMax != FIntVector::ZeroValue;

		FDeltaBatchHeader& BatchHeader = OutBatchHeaders.AddDefaulted_GetRef();
		BatchHeader.DataOffset = OutCompressedVertices.Num() * sizeof(uint32);
		BatchHeader.bTangents = bBatchHasTangents;
		BatchHeader.NumElements = NumElements;
		BatchHeader.IndexBits = FMath::CeilLogTwo(IndexDelta + 1);
		BatchHeader.PositionBits.X = FMath::CeilLogTwo(static_cast<uint32>(PositionDelta.X) + 1);
		BatchHeader.PositionBits.Y = FMath::CeilLogTwo(static_cast<uint32>(PositionDelta.Y) + 1);
		BatchHeader.PositionBits.Z = FMath::CeilLogTwo(static_cast<uint32>(PositionDelta.Z) + 1);
		BatchHeader.TangentZBits.X = FMath::CeilLogTwo(static_cast<uint32>(TangentZDelta.X) + 1);
		BatchHeader.TangentZBits.Y = FMath::CeilLogTwo(static_cast<uint32>(TangentZDelta.Y) + 1);
		BatchHeader.TangentZBits.Z = FMath::CeilLogTwo(static_cast<uint32>(TangentZDelta.Z) + 1);
		check(BatchHeader.IndexBits <= IndexMaxBits);
		check(BatchHeader.PositionBits.X <= PositionMaxBits);
		check(BatchHeader.PositionBits.Y <= PositionMaxBits);
		check(BatchHeader.PositionBits.Z <= PositionMaxBits);
		check(BatchHeader.TangentZBits.X <= TangentZMaxBits);
		check(BatchHeader.TangentZBits.Y <= TangentZMaxBits);
		check(BatchHeader.TangentZBits.Z <= TangentZMaxBits);
		BatchHeader.IndexMin = IndexMin;
		BatchHeader.PositionMin = PositionMin;
		BatchHeader.TangentZMin = TangentZMin;

		// Write quantized bits
		FDwordBitWriter BitWriter(OutCompressedVertices);
		for (uint32 LocalElementIndex = 0; LocalElementIndex < NumElements; LocalElementIndex++)
		{
			const FQuantizedDelta& Delta = QuantizedDeltas[BatchFirstElementIndex + LocalElementIndex];
			const uint32 AdjustedIndex = Delta.Index - LocalElementIndex;
			BitWriter.PutBits(AdjustedIndex - IndexMin, BatchHeader.IndexBits);
			BitWriter.PutBits(static_cast<uint32>(Delta.Position.X - PositionMin.X), BatchHeader.PositionBits.X);
			BitWriter.PutBits(static_cast<uint32>(Delta.Position.Y - PositionMin.Y), BatchHeader.PositionBits.Y);
			BitWriter.PutBits(static_cast<uint32>(Delta.Position.Z - PositionMin.Z), BatchHeader.PositionBits.Z);
			if (bBatchHasTangents)
			{
				BitWriter.PutBits(static_cast<uint32>(Delta.TangentZ.X - TangentZMin.X), BatchHeader.TangentZBits.X);
				BitWriter.PutBits(static_cast<uint32>(Delta.TangentZ.Y - TangentZMin.Y), BatchHeader.TangentZBits.Y);
				BitWriter.PutBits(static_cast<uint32>(Delta.TangentZ.Z - TangentZMin.Z), BatchHeader.TangentZBits.Z);
			}
		}
		BitWriter.Flush();
	}
}

static FMorphTargetDelta DecodeMorphTargetDelta(
	const FDeltaBatchHeader& InHeader,
	FDwordBitReader& InReader,
	uint32 InLocalIndex, 
	const float InPositionPrecision,
	const float InTangentZPrecision
	)
{
	FMorphTargetDelta Result;
	
	Result.SourceIdx = InReader.GetBits(InHeader.IndexBits) + InHeader.IndexMin + InLocalIndex;
	
	Result.PositionDelta.X = static_cast<float>(static_cast<int32>(InReader.GetBits(InHeader.PositionBits.X)) + InHeader.PositionMin.X) * InPositionPrecision;
	Result.PositionDelta.Y = static_cast<float>(static_cast<int32>(InReader.GetBits(InHeader.PositionBits.Y)) + InHeader.PositionMin.Y) * InPositionPrecision;
	Result.PositionDelta.Z = static_cast<float>(static_cast<int32>(InReader.GetBits(InHeader.PositionBits.Z)) + InHeader.PositionMin.Z) * InPositionPrecision;
	
	if (InHeader.bTangents)
	{
		Result.TangentZDelta.X = static_cast<float>(static_cast<int32>(InReader.GetBits(InHeader.TangentZBits.X)) + InHeader.TangentZMin.X) * InTangentZPrecision;
		Result.TangentZDelta.Y = static_cast<float>(static_cast<int32>(InReader.GetBits(InHeader.TangentZBits.Y)) + InHeader.TangentZMin.Y) * InTangentZPrecision;
		Result.TangentZDelta.Z = static_cast<float>(static_cast<int32>(InReader.GetBits(InHeader.TangentZBits.Z)) + InHeader.TangentZMin.Z) * InTangentZPrecision;
	}
	
	return Result;
}


void Decode(
	TConstArrayView<FDeltaBatchHeader> InBatchHeaders,
	TConstArrayView<uint32> InCompressedVertices,
	const float InPositionPrecision,
	const float InTangentZPrecision, 
	TArray<FMorphTargetDelta>& OutMorphDeltas
	)
{
	// Tally up how many deltas we'll need.
	int32 NumDeltas = 0;
	for (const FDeltaBatchHeader& Header: InBatchHeaders)
	{
		NumDeltas += static_cast<int32>(Header.NumElements);
	}

	OutMorphDeltas.Reset(NumDeltas);

	FDwordBitReader Reader(InCompressedVertices);
	
	for (const FDeltaBatchHeader& Header: InBatchHeaders)
	{
		for (uint32 ElementIndex = 0; ElementIndex < Header.NumElements; ElementIndex++)
		{
			OutMorphDeltas.Add(DecodeMorphTargetDelta(Header, Reader, ElementIndex, InPositionPrecision, InTangentZPrecision));
		}
	}
}


bool IterativeDecode(
	uint64& InOutNextItemToken,
	TConstArrayView<FDeltaBatchHeader> InBatchHeaders,
	TConstArrayView<uint32> InCompressedVertices,
	const float InPositionPrecision,
	const float InTangentZPrecision,
	FMorphTargetDelta& OutMorphTargetDelta
	)
{
	// Make sure this is not the same as uint64::max() since the iterator in MorphTarget.h uses that as an
	// iterator invalidation.
	static constexpr uint64 EndOfStreamToken = 0xEFFFFFFFFFFFFFFFull;
	
	// We've reached the end.
	if (InOutNextItemToken == EndOfStreamToken)
	{
		return false;
	}

	// We trust the token is intact.
	uint32 HeaderIndex = static_cast<uint32>(InOutNextItemToken) >> BatchSizeBits;
	uint32 ElementIndex = InOutNextItemToken & (BatchSize - 1);
	uint32 DataOffset =  InOutNextItemToken >> 32;
	
	// Less-than-or-equal, since the remaining data could be all zero number of bits.  
	checkSlow(DataOffset <= static_cast<uint32>(InCompressedVertices.Num() * 32));
	checkSlow(HeaderIndex < static_cast<uint32>(InBatchHeaders.Num()));
	
	const FDeltaBatchHeader& BatchHeader = InBatchHeaders[HeaderIndex];
	checkSlow(ElementIndex < BatchHeader.NumElements);
	
	FDwordBitReader Reader(InCompressedVertices, DataOffset);
	OutMorphTargetDelta = DecodeMorphTargetDelta(BatchHeader, Reader, ElementIndex, InPositionPrecision, InTangentZPrecision);
	
	DataOffset = Reader.GetOffset();
	
	ElementIndex++;
	if (ElementIndex == BatchHeader.NumElements)
	{
		ElementIndex = 0;
		HeaderIndex++;
	}

	if (HeaderIndex == InBatchHeaders.Num())
	{
		InOutNextItemToken = EndOfStreamToken;
	}
	else
	{
		InOutNextItemToken = (static_cast<uint64>(DataOffset) << 32) | (static_cast<uint64>(HeaderIndex) << BatchSizeBits) | ElementIndex;
	}
	
	return true;	
}


void WriteHeader(const FDeltaBatchHeader& InBatchHeader, TArrayView<uint32> OutData)
{
	FDwordViewBitWriter BitWriter(OutData);
				
	BitWriter.PutBits(InBatchHeader.DataOffset,    32);
	BitWriter.PutBits(InBatchHeader.IndexBits,      5);
	BitWriter.PutBits(InBatchHeader.PositionBits.X, 5);
	BitWriter.PutBits(InBatchHeader.PositionBits.Y, 5);
	BitWriter.PutBits(InBatchHeader.PositionBits.Z, 5);
	BitWriter.PutBits(InBatchHeader.bTangents,      1);
	BitWriter.PutBits(InBatchHeader.NumElements,   11);
	BitWriter.PutBits(InBatchHeader.IndexMin,      32);
	BitWriter.PutBits(InBatchHeader.PositionMin.X, 32);
	BitWriter.PutBits(InBatchHeader.PositionMin.Y, 32);
	BitWriter.PutBits(InBatchHeader.PositionMin.Z, 32);
	BitWriter.PutBits(InBatchHeader.TangentZBits.X, 5);
	BitWriter.PutBits(InBatchHeader.TangentZBits.Y, 5);
	BitWriter.PutBits(InBatchHeader.TangentZBits.Z, 5);
	BitWriter.PutBits(0,                           17); // Padding
	BitWriter.PutBits(InBatchHeader.TangentZMin.X, 32);
	BitWriter.PutBits(InBatchHeader.TangentZMin.Y, 32);
	BitWriter.PutBits(InBatchHeader.TangentZMin.Z, 32);
	
	BitWriter.Flush();
}

void ReadHeader(FDeltaBatchHeader& OutBatchHeader, TConstArrayView<uint32> InData)
{
	FDwordBitReader BitReader(InData);
	
	OutBatchHeader.DataOffset     = BitReader.GetBits(32);
	OutBatchHeader.IndexBits      = BitReader.GetBits(5);
	OutBatchHeader.PositionBits.X = BitReader.GetBits(5);
	OutBatchHeader.PositionBits.Y = BitReader.GetBits(5);
	OutBatchHeader.PositionBits.Z = BitReader.GetBits(5);
	OutBatchHeader.bTangents      = static_cast<bool>(BitReader.GetBits(1));
	OutBatchHeader.NumElements    = BitReader.GetBits(11);
	OutBatchHeader.IndexMin       = BitReader.GetBits(32);
	OutBatchHeader.PositionMin.X  = BitReader.GetBits(32);
	OutBatchHeader.PositionMin.Y  = BitReader.GetBits(32);
	OutBatchHeader.PositionMin.Z  = BitReader.GetBits(32);
	OutBatchHeader.TangentZBits.X = BitReader.GetBits(5);
	OutBatchHeader.TangentZBits.Y = BitReader.GetBits(5);
	OutBatchHeader.TangentZBits.Z = BitReader.GetBits(5);
									BitReader.GetBits(17); // Padding
	OutBatchHeader.TangentZMin.X  = BitReader.GetBits(32);
	OutBatchHeader.TangentZMin.Y  = BitReader.GetBits(32);
	OutBatchHeader.TangentZMin.Z  = BitReader.GetBits(32);
}

void WriteQuantizedDeltas(
		TConstArrayView<FQuantizedDelta> InQuantizedDeltas, 
		const FDeltaBatchHeader& InBatchHeader, 
		TArrayView<uint32> OutData)
{ 
	FDwordViewBitWriter BitWriter(OutData);
	
	const int32 NumElements = static_cast<int32>(InBatchHeader.NumElements);

	check(NumElements <= InQuantizedDeltas.Num());
	for (int32 LocalElementIndex = 0; LocalElementIndex < NumElements; ++LocalElementIndex)
	{
		const FQuantizedDelta& Delta = InQuantizedDeltas[LocalElementIndex];
		
		BitWriter.PutBits(Delta.Index - LocalElementIndex - InBatchHeader.IndexMin, InBatchHeader.IndexBits);
		BitWriter.PutBits(uint32(Delta.Position.X - InBatchHeader.PositionMin.X), InBatchHeader.PositionBits.X);
		BitWriter.PutBits(uint32(Delta.Position.Y - InBatchHeader.PositionMin.Y), InBatchHeader.PositionBits.Y);
		BitWriter.PutBits(uint32(Delta.Position.Z - InBatchHeader.PositionMin.Z), InBatchHeader.PositionBits.Z);

		if (InBatchHeader.bTangents)
		{
			BitWriter.PutBits(uint32(Delta.TangentZ.X - InBatchHeader.TangentZMin.X), InBatchHeader.TangentZBits.X);
			BitWriter.PutBits(uint32(Delta.TangentZ.Y - InBatchHeader.TangentZMin.Y), InBatchHeader.TangentZBits.Y);
			BitWriter.PutBits(uint32(Delta.TangentZ.Z - InBatchHeader.TangentZMin.Z), InBatchHeader.TangentZBits.Z);
		}
	}

	BitWriter.Flush();
}

void ReadQuantizedDeltas(
		TArrayView<FQuantizedDelta> OutQuantizedDeltas, 
		const FDeltaBatchHeader& InBatchHeader, 
		TConstArrayView<uint32> InData)
{
	check(OutQuantizedDeltas.Num() >= (int32)InBatchHeader.NumElements)	

	FDwordBitReader BitReader(InData);
	
	const int32 NumBatchElems = (int32)InBatchHeader.NumElements;
	for (int32 LocalElementIndex = 0; LocalElementIndex < NumBatchElems; ++LocalElementIndex)
	{
		FQuantizedDelta& QuantizedDelta = OutQuantizedDeltas[LocalElementIndex];
		
		QuantizedDelta.Index = BitReader.GetBits(InBatchHeader.IndexBits) + LocalElementIndex + InBatchHeader.IndexMin;
		QuantizedDelta.Position.X = BitReader.GetBits(InBatchHeader.PositionBits.X) + InBatchHeader.PositionMin.X;
		QuantizedDelta.Position.Y = BitReader.GetBits(InBatchHeader.PositionBits.Y) + InBatchHeader.PositionMin.Y;
		QuantizedDelta.Position.Z = BitReader.GetBits(InBatchHeader.PositionBits.Z) + InBatchHeader.PositionMin.Z;

		if (InBatchHeader.bTangents)
		{
			QuantizedDelta.TangentZ.X = BitReader.GetBits(InBatchHeader.TangentZBits.X) + InBatchHeader.TangentZMin.X;
			QuantizedDelta.TangentZ.Y = BitReader.GetBits(InBatchHeader.TangentZBits.Y) + InBatchHeader.TangentZMin.Y;
			QuantizedDelta.TangentZ.Z = BitReader.GetBits(InBatchHeader.TangentZBits.Z) + InBatchHeader.TangentZMin.Z;
		}

		OutQuantizedDeltas[LocalElementIndex] = QuantizedDelta;
	}
}

uint32 CalculateBatchDwords(const FDeltaBatchHeader& InBatchHeader)
{
	const uint32 ElementSize = InBatchHeader.IndexBits +
		InBatchHeader.PositionBits.X + InBatchHeader.PositionBits.Y + InBatchHeader.PositionBits.Z +
		(InBatchHeader.bTangents ? InBatchHeader.TangentZBits.X + InBatchHeader.TangentZBits.Y + InBatchHeader.TangentZBits.Z : 0);

	return FMath::DivideAndRoundUp(ElementSize * InBatchHeader.NumElements, 32u);
}

void QuantizeDelta(
	const FMorphTargetDelta& InDelta, 
	const bool bInNeedsTangent, 
	FQuantizedDelta& OutQuantizedDelta, 
	const float InPositionPrecision, 
	const float InTangentZPrecision)
{
	const float RcpPositionPrecision = 1.0f / InPositionPrecision;
	const float RcpTangentZPrecision = 1.0f / InTangentZPrecision;

	const FVector3f& PositionDelta = InDelta.PositionDelta;
	OutQuantizedDelta.Position.X = FMath::RoundToInt(FMath::Clamp(PositionDelta.X * RcpPositionPrecision, PositionMinValue, PositionMaxValue));
	OutQuantizedDelta.Position.Y = FMath::RoundToInt(FMath::Clamp(PositionDelta.Y * RcpPositionPrecision, PositionMinValue, PositionMaxValue));
	OutQuantizedDelta.Position.Z = FMath::RoundToInt(FMath::Clamp(PositionDelta.Z * RcpPositionPrecision, PositionMinValue, PositionMaxValue));

	const FVector3f TangentZDelta = bInNeedsTangent ? InDelta.TangentZDelta : FVector3f::ZeroVector;
	OutQuantizedDelta.TangentZ.X = FMath::RoundToInt(FMath::Clamp(TangentZDelta.X * RcpTangentZPrecision, TangentZMinValue, TangentZMaxValue));
	OutQuantizedDelta.TangentZ.Y = FMath::RoundToInt(FMath::Clamp(TangentZDelta.Y * RcpTangentZPrecision, TangentZMinValue, TangentZMaxValue));
	OutQuantizedDelta.TangentZ.Z = FMath::RoundToInt(FMath::Clamp(TangentZDelta.Z * RcpTangentZPrecision, TangentZMinValue, TangentZMaxValue));

	OutQuantizedDelta.Index = InDelta.SourceIdx;
}

void DequantizeDelta(
	FMorphTargetDelta& OutDelta, 
	const bool bInNeedsTangent, 
	const FQuantizedDelta& InQuantizedDelta, 
	const float InPositionPrecision, 
	const float InTangentZPrecision)
{
	OutDelta.SourceIdx = InQuantizedDelta.Index;
	OutDelta.PositionDelta.X = InQuantizedDelta.Position.X * InPositionPrecision;
	OutDelta.PositionDelta.Y = InQuantizedDelta.Position.Y * InPositionPrecision;
	OutDelta.PositionDelta.Z = InQuantizedDelta.Position.Z * InPositionPrecision;

	if (bInNeedsTangent)
	{
		OutDelta.TangentZDelta.X = InQuantizedDelta.TangentZ.X * InTangentZPrecision;
		OutDelta.TangentZDelta.Y = InQuantizedDelta.TangentZ.Y * InTangentZPrecision;
		OutDelta.TangentZDelta.Z = InQuantizedDelta.TangentZ.Z * InTangentZPrecision;
	}
	else
	{
		OutDelta.TangentZDelta = FVector3f::ZeroVector;
	}
}

} // namespace UE::MorphTargetVertexCodec
