// Copyright Epic Games, Inc. All Rights Reserved.

#include "Rendering/MorphTargetVertexInfoBuffers.h"
#include "ProfilingDebugging/LoadTimeTracker.h"
#include "Rendering/SkeletalMeshLODRenderData.h"
#include "Animation/MorphTarget.h"
#include "Containers/StaticArray.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "RHIResourceUtils.h"

extern int32 GSkinCacheRecomputeTangents;

FMorphTargetVertexInfoBuffers::FMorphTargetVertexInfoBuffers() = default;
FMorphTargetVertexInfoBuffers::~FMorphTargetVertexInfoBuffers() = default;

void FMorphTargetVertexInfoBuffers::InitRHI(FRHICommandListBase& RHICmdList)
{
	SCOPED_LOADTIMER(FFMorphTargetVertexInfoBuffers_InitRHI);

	check(NumTotalBatches > 0);
	check(!bRHIInitialized);

	const static FLazyName ClassName(TEXT("FMorphTargetVertexInfoBuffers"));

	const FRHIBufferCreateDesc Desc =
		FRHIBufferCreateDesc::CreateStructured<uint32>(TEXT("MorphData"), MorphData.Num())
		.AddUsage(BUF_Static | BUF_ByteAddressBuffer | BUF_ShaderResource)
		.SetInitialState(ERHIAccess::SRVMask)
		.SetClassName(ClassName)
		.SetOwnerName(GetOwnerName())
		.SetInitActionResourceArray(&MorphData);

	MorphDataBuffer = RHICmdList.CreateBuffer(Desc);
	MorphDataSRV = RHICmdList.CreateShaderResourceView(MorphDataBuffer, FRHIViewDesc::CreateBufferSRV().SetTypeFromBuffer(MorphDataBuffer));

	if (bEmptyMorphCPUDataOnInitRHI)
	{
		MorphData.Empty();
		bIsMorphCPUDataValid = false;
	}

	bRHIInitialized = true;
}

void FMorphTargetVertexInfoBuffers::ReleaseRHI()
{
	MorphDataBuffer.SafeRelease();
	MorphDataSRV.SafeRelease();
	bRHIInitialized = false;
}

uint32 FMorphTargetVertexInfoBuffers::GetMaximumThreadGroupSize()
{
	//D3D11 there can be at most 65535 Thread Groups in each dimension of a Dispatch call.
	uint64 MaximumThreadGroupSize = uint64(GMaxComputeDispatchDimension) * 32ull;
	return uint32(FMath::Min<uint64>(MaximumThreadGroupSize, UINT32_MAX));
}

void FMorphTargetVertexInfoBuffers::ResetCPUData()
{
	MorphData.Empty();
	MaximumValuePerMorph.Empty();
	MinimumValuePerMorph.Empty();
	BatchStartOffsetPerMorph.Empty();
	BatchesPerMorph.Empty();
	NumTotalBatches = 0;
	PositionPrecision = 0.0f;
	TangentZPrecision = 0.0f;
	bResourcesInitialized = false;
	bIsMorphCPUDataValid = false;
}

void FMorphTargetVertexInfoBuffers::ValidateVertexBuffers(bool bMorphTargetsShouldBeValid)
{
#if DO_CHECK
	check(BatchesPerMorph.Num() == BatchStartOffsetPerMorph.Num());
	check(BatchesPerMorph.Num() == MaximumValuePerMorph.Num());
	check(BatchesPerMorph.Num() == MinimumValuePerMorph.Num());

	if (bMorphTargetsShouldBeValid)
	{
		if (NumTotalBatches > 0)
		{
			check(MorphData.Num() > 0);
		}
		else
		{
			check(MorphData.Num() == 0);
		}
	}
	
#endif //DO_CHECK
}

void FMorphTargetVertexInfoBuffers::Serialize(FArchive& Ar)
{
	if (Ar.IsSaving())
	{
		check(bResourcesInitialized);
		check(bIsMorphCPUDataValid);
		ValidateVertexBuffers(true);
	}
	else if(Ar.IsLoading())
	{
		ResetCPUData();
	}

	Ar << MorphData
	   << MinimumValuePerMorph
	   << MaximumValuePerMorph
	   << BatchStartOffsetPerMorph
	   << BatchesPerMorph
	   << NumTotalBatches
	   << PositionPrecision
	   << TangentZPrecision;

	if (Ar.IsLoading())
	{
		bRHIInitialized = false;
		bIsMorphCPUDataValid = true;
		bResourcesInitialized = true;
		ValidateVertexBuffers(true);
	}
}

FArchive& operator<<(FArchive& Ar, FMorphTargetVertexInfoBuffers& MorphTargetVertexInfoBuffers)
{
	MorphTargetVertexInfoBuffers.Serialize(Ar);
	return Ar;
}

void FMorphTargetVertexInfoBuffers::InitMorphResources(EShaderPlatform ShaderPlatform, const TArray<FSkelMeshRenderSection>& RenderSections, const TArray<UMorphTarget*>& MorphTargets, int32 NumVertices, int32 LODIndex, float TargetPositionErrorTolerance)
{
	check(!IsRHIInitialized());
	check(!IsMorphResourcesInitialized());
	check(!IsMorphCPUDataValid());

	bResourcesInitialized = true;

	// UseGPUMorphTargets() can be toggled only on SM5 atm
	if (!IsPlatformShaderSupported(ShaderPlatform) || MorphTargets.Num() == 0)
	{
		return;
	}

	bIsMorphCPUDataValid = true;

	TArray<const FMorphTargetLODModel*> MorphTargetLODs;
	MorphTargetLODs.Reserve(MorphTargets.Num());
	
	TBitArray UsesBuiltinMorphTargetCompression;
	UsesBuiltinMorphTargetCompression.Reserve(MorphTargets.Num());

	for (UMorphTarget* MorphTarget : MorphTargets)
	{
		TArray<FMorphTargetLODModel>& MorphLODModels = MorphTarget->GetMorphLODModels();
		
		const FMorphTargetLODModel* MorphTargetLOD = MorphLODModels.IsValidIndex(LODIndex) ? &MorphLODModels[LODIndex] : nullptr;
		MorphTargetLODs.Add(MorphTargetLOD);
		UsesBuiltinMorphTargetCompression.Add(MorphTarget->UsesBuiltinMorphTargetCompression());
	}

	Compress(RenderSections, MorphTargetLODs, UsesBuiltinMorphTargetCompression, NumVertices, TargetPositionErrorTolerance);
	
	ValidateVertexBuffers(true);
}

void FMorphTargetVertexInfoBuffers::InitMorphResourcesStreaming(EShaderPlatform ShaderPlatform, const TArray<FSkelMeshRenderSection>& RenderSections, TConstArrayView<const FMorphTargetLODModel*> MorphTargets, int32 NumVertices, float TargetPositionErrorTolerance)
{
	ResetCPUData();

	// UseGPUMorphTargets() can be toggled only on SM5 atm
	if (!IsPlatformShaderSupported(ShaderPlatform) || MorphTargets.Num() == 0)
	{
		return;
	}

	const TBitArray UsesBuiltinMorphTargetCompression(true, MorphTargets.Num());
	Compress(RenderSections, MorphTargets, UsesBuiltinMorphTargetCompression, NumVertices, TargetPositionErrorTolerance);
	
	bRHIInitialized = false;
	bIsMorphCPUDataValid = true;
	bResourcesInitialized = true;
	ValidateVertexBuffers(true);
}

#if !WITH_EDITOR
void FMorphTargetVertexInfoBuffers::InitMorphResourcesStreaming(
		EShaderPlatform ShaderPlatform,
		TConstArrayView<const FMorphTargetCompressedLODModel*> InCompressedMorphTargets, 
		TConstArrayView<FVector4f> InMinimumValuePerMorph,  
		TConstArrayView<FVector4f> InMaximumValuePerMorph)
{
	ResetCPUData();

	// UseGPUMorphTargets() can be toggled only on SM5 atm
	if (!IsPlatformShaderSupported(ShaderPlatform) || InCompressedMorphTargets.Num() == 0)
	{
		return;
	}

	InitDataFromCompressedMorphs(InCompressedMorphTargets, InMinimumValuePerMorph, InMaximumValuePerMorph);

	bRHIInitialized = false;
	bIsMorphCPUDataValid = true;
	bResourcesInitialized = true;
	ValidateVertexBuffers(true);
}

void FMorphTargetVertexInfoBuffers::InitDataFromCompressedMorphs(
		TConstArrayView<const FMorphTargetCompressedLODModel*> InCompressedMorphTargets, 
		TConstArrayView<FVector4f> InMinimumValuePerMorph,  
		TConstArrayView<FVector4f> InMaximumValuePerMorph)
{
	// Assum all morphs targets have the same PositionPrecision and TangentZPrecision.
	{
		PositionPrecision = 0.0f;
		TangentZPrecision = 0.0f;

		int32 MorphIndex = 0;
		for (; MorphIndex < InCompressedMorphTargets.Num(); ++MorphIndex)
		{
			const FMorphTargetCompressedLODModel* CompressedMorph = InCompressedMorphTargets[MorphIndex];
			if (CompressedMorph)
			{
				PositionPrecision = CompressedMorph->PositionPrecision;
				TangentZPrecision = CompressedMorph->TangentPrecision;
				break;
			}
		}

#if DO_CHECK
		for (; MorphIndex < InCompressedMorphTargets.Num(); ++MorphIndex)
		{
			const FMorphTargetCompressedLODModel* CompressedMorph = InCompressedMorphTargets[MorphIndex];
			if (CompressedMorph)
			{
				check(PositionPrecision == CompressedMorph->PositionPrecision);
				check(TangentZPrecision == CompressedMorph->TangentPrecision);
			}
		}
#endif
	}	

	BatchStartOffsetPerMorph.Empty(InCompressedMorphTargets.Num());
	BatchesPerMorph.Empty(InCompressedMorphTargets.Num());
	MaximumValuePerMorph.Empty(InCompressedMorphTargets.Num());
	MinimumValuePerMorph.Empty(InCompressedMorphTargets.Num());

	check(InMaximumValuePerMorph.Num() == InCompressedMorphTargets.Num());
	check(InMinimumValuePerMorph.Num() == InCompressedMorphTargets.Num());

	// Compute allocation size and add metadata.
	int32 TotalDataSizeInDwords = 0;
	NumTotalBatches = 0;

	int32 CurrentMorphBatchStartOffsetInBatches = 0;
	for (int32 MorphIndex = 0; MorphIndex < InCompressedMorphTargets.Num(); ++MorphIndex)
	{
		using namespace UE::MorphTargetVertexCodec;
		
		const FMorphTargetCompressedLODModel* CompressedMorph = InCompressedMorphTargets[MorphIndex];

		if (CompressedMorph)
		{
			const int32 MorphTargetNumBatches = CompressedMorph->PackedDeltaHeaders.Num();
		
			BatchesPerMorph.Add(MorphTargetNumBatches);
			BatchStartOffsetPerMorph.Add(CurrentMorphBatchStartOffsetInBatches);
			MinimumValuePerMorph.Add(InMinimumValuePerMorph[MorphIndex]);
			MaximumValuePerMorph.Add(InMaximumValuePerMorph[MorphIndex]);

			CurrentMorphBatchStartOffsetInBatches += MorphTargetNumBatches;

			NumTotalBatches += MorphTargetNumBatches;
			TotalDataSizeInDwords += 
					MorphTargetNumBatches * UE::MorphTargetVertexCodec::NumBatchHeaderDwords + 
					CompressedMorph->PackedDeltaData.Num();

			static_assert(sizeof(decltype(CompressedMorph->PackedDeltaData)::ElementType) == sizeof(uint32));
		}
		else
		{
			BatchesPerMorph.Add(0);
			BatchStartOffsetPerMorph.Add(CurrentMorphBatchStartOffsetInBatches);
			MinimumValuePerMorph.Add(FVector4f(0.0f, 0.0f, 0.0f, 0.0f));
			MaximumValuePerMorph.Add(FVector4f(0.0f, 0.0f, 0.0f, 0.0f));
		}
	}

	// Add extra padding to make sure it is always safe to access the data with load4s.
	MorphData.Empty(TotalDataSizeInDwords > 0 ? TotalDataSizeInDwords + 3 : 0);

	// Fixup and append headers to the bitstream.
	int32 AccumulatedDataOffsetFixupInBytes = NumTotalBatches * UE::MorphTargetVertexCodec::NumBatchHeaderDwords * sizeof(uint32);
	for (const FMorphTargetCompressedLODModel* CompressedMorph : InCompressedMorphTargets)
	{
		using namespace UE::MorphTargetVertexCodec;

		if (CompressedMorph)
		{
			for (const FDeltaBatchHeader& BatchHeader : CompressedMorph->PackedDeltaHeaders)
			{
				TStaticArray<uint32, NumBatchHeaderDwords> HeaderData;
				WriteHeader(BatchHeader, HeaderData);

				// Fixup data offsets. 
				// First header dword corresponds to the DataOffset. See UE::MorphTargetVertexCodec::WriteHeader.
				HeaderData[0] += AccumulatedDataOffsetFixupInBytes;
				MorphData.Append(HeaderData);
			}	

			AccumulatedDataOffsetFixupInBytes += CompressedMorph->PackedDeltaData.NumBytes();
		}
	}

	// Append deltas data to the bitstream.
	for (const FMorphTargetCompressedLODModel* CompressedMorph : InCompressedMorphTargets)
	{
		if (CompressedMorph)
		{
			MorphData.Append(CompressedMorph->PackedDeltaData);
		}
	}

	check(TotalDataSizeInDwords == MorphData.Num());

	if (MorphData.Num() > 0)
	{
		// Pad to make sure it is always safe to access the data with load4s.
		MorphData.Add(0u);
		MorphData.Add(0u);
		MorphData.Add(0u);
	}
}
#endif

void FMorphTargetVertexInfoBuffers::Compress(const TArray<FSkelMeshRenderSection>& RenderSections, TConstArrayView<const FMorphTargetLODModel*> MorphTargets, const TBitArray<>& UsesBuiltinMorphTargetCompression, int32 NumVertices, float TargetPositionErrorTolerance)
{
	using namespace UE::MorphTargetVertexCodec;
	
	PositionPrecision = ComputePositionPrecision(TargetPositionErrorTolerance);
	TangentZPrecision = ComputeTangentPrecision();

	MorphData.Empty();
	
	BatchStartOffsetPerMorph.Empty(MorphTargets.Num());
	BatchesPerMorph.Empty(MorphTargets.Num());
	MaximumValuePerMorph.Empty(MorphTargets.Num());
	MinimumValuePerMorph.Empty(MorphTargets.Num());

	// Mark vertices that are in a section that doesn't recompute tangents as needing tangents
	TBitArray<> VertexNeedsTangents;
	VertexNeedsTangents.Init(false, NumVertices);
	for (const FSkelMeshRenderSection& RenderSection : RenderSections)
	{
		const bool bNeedsTangents = FMorphTargetVertexInfoBuffers::NeedsTangents(RenderSection.bRecomputeTangent);
		if (bNeedsTangents)
		{
			VertexNeedsTangents.SetRange(RenderSection.BaseVertexIndex, RenderSection.NumVertices, true);
		}
	}
	
	// Populate the arrays to be filled in later in the render thread
	TArray<FDeltaBatchHeader> BatchHeaders;
	TArray<uint32> BitstreamData;
	for (int32 AnimIdx = 0; AnimIdx < MorphTargets.Num(); ++AnimIdx)
	{
		uint32 BatchStartOffset = BatchHeaders.Num();

		float MaximumValues[4] = { -FLT_MAX, -FLT_MAX, -FLT_MAX, -FLT_MAX };
		float MinimumValues[4] = { +FLT_MAX, +FLT_MAX, +FLT_MAX, +FLT_MAX };

		const FMorphTargetLODModel* MorphModel = MorphTargets[AnimIdx];
		int32 NumSrcDeltas = MorphModel ? MorphModel->Vertices.Num() : 0;
		
		if (NumSrcDeltas == 0 || !UsesBuiltinMorphTargetCompression[AnimIdx])
		{
			MaximumValues[0] = 0.0f;
			MaximumValues[1] = 0.0f;
			MaximumValues[2] = 0.0f;
			MaximumValues[3] = 0.0f;

			MinimumValues[0] = 0.0f;
			MinimumValues[1] = 0.0f;
			MinimumValues[2] = 0.0f;
			MinimumValues[3] = 0.0f;
		}
		else
		{
			const FMorphTargetDelta* MorphDeltas = MorphModel->Vertices.GetData();
			
			for (int32 DeltaIndex = 0; DeltaIndex < NumSrcDeltas; ++DeltaIndex)
			{
				const FMorphTargetDelta& MorphDelta = MorphDeltas[DeltaIndex];
			
				const FVector3f TangentZDelta = (VertexNeedsTangents.IsValidIndex(MorphDelta.SourceIdx) && VertexNeedsTangents[MorphDelta.SourceIdx]) ? MorphDelta.TangentZDelta : FVector3f::ZeroVector;

				// when import, we do check threshold, and also when adding weight, we do have threshold for how smaller weight can fit in
				// so no reason to check here another threshold
				MaximumValues[0] = FMath::Max(MaximumValues[0], MorphDelta.PositionDelta.X);
				MaximumValues[1] = FMath::Max(MaximumValues[1], MorphDelta.PositionDelta.Y);
				MaximumValues[2] = FMath::Max(MaximumValues[2], MorphDelta.PositionDelta.Z);
				MaximumValues[3] = FMath::Max(MaximumValues[3], FMath::Max(TangentZDelta.X, FMath::Max(TangentZDelta.Y, TangentZDelta.Z)));

				MinimumValues[0] = FMath::Min(MinimumValues[0], MorphDelta.PositionDelta.X);
				MinimumValues[1] = FMath::Min(MinimumValues[1], MorphDelta.PositionDelta.Y);
				MinimumValues[2] = FMath::Min(MinimumValues[2], MorphDelta.PositionDelta.Z);
				MinimumValues[3] = FMath::Min(MinimumValues[3], FMath::Min(TangentZDelta.X, FMath::Min(TangentZDelta.Y, TangentZDelta.Z)));
			}

			// Encode the actual morph vertex info into the quantized bitstream.
			Encode(MorphModel->Vertices, &VertexNeedsTangents, PositionPrecision, TangentZPrecision, BatchHeaders, BitstreamData);
		}
		
		const uint32 MorphNumBatches = BatchHeaders.Num() - BatchStartOffset;
		BatchStartOffsetPerMorph.Add(BatchStartOffset);
		BatchesPerMorph.Add(MorphNumBatches);
		MaximumValuePerMorph.Add(FVector4f(MaximumValues[0], MaximumValues[1], MaximumValues[2], MaximumValues[3]));
		MinimumValuePerMorph.Add(FVector4f(MinimumValues[0], MinimumValues[1], MinimumValues[2], MinimumValues[3]));
	}

	NumTotalBatches = BatchHeaders.Num();

	// Write packed batch headers
	for (FDeltaBatchHeader& BatchHeader : BatchHeaders)
	{
		BatchHeader.DataOffset += BatchHeaders.Num() * NumBatchHeaderDwords * sizeof(uint32);

		TStaticArray<uint32, NumBatchHeaderDwords> HeaderData;
		WriteHeader(BatchHeader, HeaderData);
		MorphData.Append(HeaderData);
	}

	// Append bitstream data
	MorphData.Append(BitstreamData);

	if (MorphData.Num() > 0)
	{
		// Pad to make sure it is always safe to access the data with load4s.
		MorphData.Add(0u);
		MorphData.Add(0u);
		MorphData.Add(0u);
	}

	// UE_LOGF(LogStaticMesh, Log, "Morph compression time: [%.2fs]", FPlatformTime::ToMilliseconds(FPlatformTime::Cycles() - StartTime) / 1000.0f);
}

bool FMorphTargetVertexInfoBuffers::IsPlatformShaderSupported(EShaderPlatform ShaderPlatform)
{
	return IsFeatureLevelSupported(ShaderPlatform, ERHIFeatureLevel::SM5);
}

bool FMorphTargetVertexInfoBuffers::NeedsTangents(bool bRecomputeTangents)
{
	return !(GSkinCacheRecomputeTangents > 0 && (bRecomputeTangents || GSkinCacheRecomputeTangents == 1));
}
