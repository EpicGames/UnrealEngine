// Copyright Epic Games, Inc. All Rights Reserved.

#include "Rendering/MotionVectorSimulation.h"
#include "Components/SceneComponent.h"
#include "HAL/IConsoleManager.h"

// Global switch for enabling/disabling motion vector simulation under some circumstances (ie on camera cut frames)
int32 GMotionVectorSimulation = 0;
static FAutoConsoleVariableRef CVarEnabled(
	TEXT("r.MotionVectorSimulation"),
	GMotionVectorSimulation,
	TEXT("Controls whether to allow simulated motion vectors on scene components, geometry caches and skinned meshes on camera cut frames.")
	);

FMotionVectorSimulation::FMotionVectorSimulation()
{}

FMotionVectorSimulation::~FMotionVectorSimulation()
{}

FMotionVectorSimulation& FMotionVectorSimulation::Get()
{
	static FMotionVectorSimulation Singleton;
	return Singleton;
}

void FMotionVectorSimulation::OnUObjectArrayShutdown()
{
	SimulatedTransforms.Empty();
	GUObjectArray.RemoveUObjectDeleteListener(this);
}

bool FMotionVectorSimulation::IsEnabled()
{
	return GMotionVectorSimulation != 0;
}

void FMotionVectorSimulation::NotifyUObjectDeleted(const UObjectBase* Object, int32 Index)
{
	SimulatedTransforms.Remove(Object);
}

TOptional<FTransform> FMotionVectorSimulation::GetPreviousTransform(UObject* Object) const
{
	if (IsEnabled())
	{
		FScopeLock Lock(&MapCriticalSection);
		const FSimulatedTransform* PreviousTransform = SimulatedTransforms.Find(Object);

		// Only return the transform if it pertains to the current frame number
		if (PreviousTransform && PreviousTransform->FrameNumber == GFrameCounter)
		{
			return PreviousTransform->Transform;
		}
	}

	return TOptional<FTransform>();
}

bool FMotionVectorSimulation::GetPreviousTransform(UObject* Object, FTransform* OutTransform) const
{
	if (IsEnabled())
	{
		FScopeLock Lock(&MapCriticalSection);
		const FSimulatedTransform* PreviousTransform = SimulatedTransforms.Find(Object);

		// Only return the transform if it pertains to the current frame number
		if (PreviousTransform && PreviousTransform->FrameNumber == GFrameCounter)
		{
			*OutTransform = PreviousTransform->Transform;
			return true;
		}
	}

	return false;
}

void FMotionVectorSimulation::SetPreviousTransform(UObject* Object, const FTransform& InNewPreviousTransform)
{
	FScopeLock Lock(&MapCriticalSection);

	FSimulatedTransform SimulatedTransform;
	SimulatedTransform.Transform = InNewPreviousTransform;
	SimulatedTransform.FrameNumber = GFrameCounter;

	SimulatedTransforms.Add(Object, SimulatedTransform);
}

void FMotionVectorSimulation::ClearPreviousTransform(UObject* Object)
{
	FScopeLock Lock(&MapCriticalSection);

	SimulatedTransforms.Remove(Object);
}

void FMotionVectorSimulation::Tick(float DeltaTime)
{
	FScopeLock Lock(&MapCriticalSection);

	for (auto It = SimulatedTransforms.CreateIterator(); It; ++It)
	{
		if (It.Value().FrameNumber < GFrameCounter)
		{
			It.RemoveCurrent();
		}
	}
}

TStatId FMotionVectorSimulation::GetStatId() const
{
	RETURN_QUICK_DECLARE_CYCLE_STAT(MotionVectorSimulation, STATGROUP_Tickables);
}
