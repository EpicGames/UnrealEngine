// Copyright Epic Games, Inc. All Rights Reserved.

#include "Rendering/MultiSizeIndexContainer.h"
#include "Rendering/RenderCommandPipes.h"
#include "EngineLogs.h"
#include "RawIndexBuffer.h"
#include "Stats/Stats.h"

FMultiSizeIndexContainer::~FMultiSizeIndexContainer()
{
	if (IndexBuffer)
	{
		delete IndexBuffer;
	}
}

void FMultiSizeIndexContainer::SetOwnerName(const FName& OwnerName)
{
	check(IsInGameThread());
	if (IndexBuffer)
	{
		IndexBuffer->SetOwnerName(OwnerName);
	}
}

/**
* Initialize the index buffer's render resources.
*/
void FMultiSizeIndexContainer::InitResources()
{
	check(IsInGameThread());
	if (IndexBuffer)
	{
		BeginInitResource(IndexBuffer, &UE::RenderCommandPipe::SkeletalMesh);
	}
}

/**
* Releases the index buffer's render resources.
*/
void FMultiSizeIndexContainer::ReleaseResources()
{
	check(IsInGameThread());
	if (IndexBuffer)
	{
		BeginReleaseResource(IndexBuffer, &UE::RenderCommandPipe::SkeletalMesh);
	}
}

/**
* Creates a new index buffer
*/
void FMultiSizeIndexContainer::CreateIndexBuffer(uint8 InDataTypeSize)
{
	check(IndexBuffer == NULL);
	bool bNeedsCPUAccess = true;

	DataTypeSize = InDataTypeSize;

	if (InDataTypeSize == sizeof(uint16))
	{
		IndexBuffer = new FRawStaticIndexBuffer16or32<uint16>(bNeedsCPUAccess);
	}
	else
	{
		IndexBuffer = new FRawStaticIndexBuffer16or32<uint32>(bNeedsCPUAccess);
	}
}

/**
* Repopulates the index buffer
*/
void FMultiSizeIndexContainer::RebuildIndexBuffer(uint8 InDataTypeSize, const TArray<uint32>& NewArray)
{
	bool bNeedsCPUAccess = true;

	if (IndexBuffer)
	{
		delete IndexBuffer;
	}
	DataTypeSize = InDataTypeSize;

	if (DataTypeSize == sizeof(uint16))
	{
		IndexBuffer = new FRawStaticIndexBuffer16or32<uint16>(bNeedsCPUAccess);
	}
	else
	{
		IndexBuffer = new FRawStaticIndexBuffer16or32<uint32>(bNeedsCPUAccess);
	}

	CopyIndexBuffer(NewArray);
}

/**
* Returns a 32 bit version of the index buffer
*/
void FMultiSizeIndexContainer::GetIndexBuffer(TArray<uint32>& OutArray) const
{
	check(IndexBuffer);

	OutArray.Reset();
	int32 NumIndices = IndexBuffer->Num();
	OutArray.AddUninitialized(NumIndices);

	for (int32 I = 0; I < NumIndices; ++I)
	{
		OutArray[I] = IndexBuffer->Get(I);
	}
}

/**
* Populates the index buffer with a new set of indices
*/
void FMultiSizeIndexContainer::CopyIndexBuffer(const TArray<uint32>& NewArray)
{
	check(IndexBuffer);

	// On console the resource arrays can't have items added directly to them
	if (FPlatformProperties::HasEditorOnlyData() == false)
	{
		if (DataTypeSize == sizeof(uint16))
		{
			TArray<uint16> WordArray;
			for (int32 i = 0; i < NewArray.Num(); ++i)
			{
				WordArray.Add((uint16)NewArray[i]);
			}

			((FRawStaticIndexBuffer16or32<uint16>*)IndexBuffer)->AssignNewBuffer(WordArray);
		}
		else
		{
			((FRawStaticIndexBuffer16or32<uint32>*)IndexBuffer)->AssignNewBuffer(NewArray);
		}
	}
	else
	{
		IndexBuffer->Empty();
		for (int32 i = 0; i < NewArray.Num(); ++i)
		{
#if WITH_EDITOR
			if (DataTypeSize == sizeof(uint16) && NewArray[i] > MAX_uint16)
			{
				UE_LOGF(LogSkeletalMesh, Warning, "Attempting to copy %u into a uint16 index buffer - this value will overflow to %u, use RebuildIndexBuffer to create a uint32 index buffer!", NewArray[i], (uint16)NewArray[i]);
			}
#endif
			IndexBuffer->AddItem(NewArray[i]);
		}
	}
}

void FMultiSizeIndexContainer::Serialize(FArchive& Ar, bool bNeedsCPUAccess)
{
	DECLARE_SCOPE_CYCLE_COUNTER(TEXT("FMultiSizeIndexContainer::Serialize"), STAT_MultiSizeIndexContainer_Serialize, STATGROUP_LoadTime);
	if (Ar.UEVer() < VER_UE4_KEEP_SKEL_MESH_INDEX_DATA)
	{
		bool bOldNeedsCPUAccess = true;
		Ar << bOldNeedsCPUAccess;
	}
	Ar << DataTypeSize;

	if (!IndexBuffer)
	{
		if (DataTypeSize == sizeof(uint16))
		{
			IndexBuffer = new FRawStaticIndexBuffer16or32<uint16>(bNeedsCPUAccess);
		}
		else
		{
			IndexBuffer = new FRawStaticIndexBuffer16or32<uint32>(bNeedsCPUAccess);
		}
	}

	IndexBuffer->Serialize(Ar);
}

void FMultiSizeIndexContainer::SerializeMetaData(FArchive& Ar, bool bNeedsCPUAccess)
{
	Ar << DataTypeSize;

	if (!IndexBuffer)
	{
		if (DataTypeSize == sizeof(uint16))
		{
			IndexBuffer = new FRawStaticIndexBuffer16or32<uint16>(bNeedsCPUAccess);
		}
		else
		{
			IndexBuffer = new FRawStaticIndexBuffer16or32<uint32>(bNeedsCPUAccess);
		}
	}

	IndexBuffer->SerializeMetaData(Ar);
}

FBufferRHIRef FMultiSizeIndexContainer::CreateRHIBuffer(FRHICommandListBase& RHICmdList)
{
	if (IndexBuffer)
	{
		if (DataTypeSize == sizeof(uint16))
		{
			return static_cast<FRawStaticIndexBuffer16or32<uint16>*>(IndexBuffer)->CreateRHIBuffer(RHICmdList);
		}
		else
		{
			return static_cast<FRawStaticIndexBuffer16or32<uint32>*>(IndexBuffer)->CreateRHIBuffer(RHICmdList);
		}
	}
	return nullptr;
}

void FMultiSizeIndexContainer::InitRHIForStreaming(FRHIBuffer* IntermediateBuffer, FRHIResourceReplaceBatcher& Batcher)
{
	check(!((uint32)!!IntermediateBuffer ^ (uint32)!!IndexBuffer));
	if (IntermediateBuffer)
	{
		if (DataTypeSize == sizeof(uint16))
		{
			static_cast<FRawStaticIndexBuffer16or32<uint16>*>(IndexBuffer)->InitRHIForStreaming(IntermediateBuffer, Batcher);
		}
		else
		{
			static_cast<FRawStaticIndexBuffer16or32<uint32>*>(IndexBuffer)->InitRHIForStreaming(IntermediateBuffer, Batcher);
		}
	}
}

void FMultiSizeIndexContainer::ReleaseRHIForStreaming(FRHIResourceReplaceBatcher& Batcher)
{
	if (IndexBuffer)
	{
		if (DataTypeSize == sizeof(uint16))
		{
			static_cast<FRawStaticIndexBuffer16or32<uint16>*>(IndexBuffer)->ReleaseRHIForStreaming(Batcher);
		}
		else
		{
			static_cast<FRawStaticIndexBuffer16or32<uint32>*>(IndexBuffer)->ReleaseRHIForStreaming(Batcher);
		}
	}
}

#if WITH_EDITOR
/**
* Retrieves index buffer related data
*/
void FMultiSizeIndexContainer::GetIndexBufferData(FMultiSizeIndexContainerData& OutData) const
{
	OutData.DataTypeSize = DataTypeSize;
	GetIndexBuffer(OutData.Indices);
}

FMultiSizeIndexContainer::FMultiSizeIndexContainer(const FMultiSizeIndexContainer& Other)
	: DataTypeSize(sizeof(uint16))
	, IndexBuffer(nullptr)
{
	// Cant copy this index buffer, assumes it will be rebuilt later
	IndexBuffer = nullptr;
}

FMultiSizeIndexContainer& FMultiSizeIndexContainer::operator=(const FMultiSizeIndexContainer& Buffer)
{
	// Cant copy this index buffer.  Delete the index buffer type.
	// assumes it will be rebuilt later
	if (IndexBuffer)
	{
		delete IndexBuffer;
		IndexBuffer = nullptr;
	}

	return *this;
}
#endif
