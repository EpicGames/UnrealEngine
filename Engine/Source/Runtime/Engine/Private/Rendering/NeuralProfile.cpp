// Copyright Epic Games, Inc. All Rights Reserved.

#include "Engine/NeuralProfile.h"
#include "Engine/Texture2D.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "Math/Float16.h"
#include "Rendering/BurleyNormalizedSSS.h"
#include "EngineModule.h"
#include "RenderTargetPool.h"
#include "PixelShaderUtils.h"
#include "RenderingThread.h"
#include "Rendering/Texture2DResource.h"
#include "RenderGraphResources.h"
#include "RenderGraphBuilder.h"


#include UE_INLINE_GENERATED_CPP_BY_NAME(NeuralProfile)

DEFINE_LOG_CATEGORY_STATIC(LogNeuralProfile, Log, All);

TUniquePtr<NeuralProfile::INeuralProfileManager> GNeuralProfileManager;

static const TCHAR* GetNeuralProfileRuntimeName(ENeuralProfileRuntimeType NeuralProfileRuntimeType)
{
	static const TCHAR* const kRuntimeNames[] = {
		TEXT("NNERuntimeORTDml"),
		TEXT("NNERuntimeRDGHlsl")
	};

	static_assert(UE_ARRAY_COUNT(kRuntimeNames) == int32(ENeuralProfileRuntimeType::MAX), "Fix me");
	return kRuntimeNames[int32(NeuralProfileRuntimeType)];
}

///////////////////////////////////////////////////////////////////////////////////////////////////
// FNeuralProfileModelTextureManager

//@TODO: add texture management.
class FNeuralProfileModelTextureManager : public FRenderResource
{
public:
	FNeuralProfileModelTextureManager()
	{
		check(IsInGameThread());
		//@TODO: add a default network?
		//		 add param texture

		//FProfileEntry& Entry = NeuralProfileEntries.AddDefaulted_GetRef();
		//Entry.Settings = FNeuralProfileStruct();
		//Entry.Profile = nullptr;
	}

	~FNeuralProfileModelTextureManager() {}

	// convenience, can be optimized 
	// @param Profile must not be nullptr, game thread pointer, do not dereference, only for comparison
	int32 AddOrUpdateProfile(const UNeuralProfile* InProfile, const FGuid& InGuid, const FNeuralProfileStruct InSettings)
	{
		check(InProfile);
		
		int32 AllocationId = FindAllocationId(InProfile);

		if (AllocationId != INDEX_NONE)
		{
			UpdateProfile(AllocationId, InSettings);
		}
		else
		{
			AllocationId = AddProfile(InProfile, InGuid, InSettings);
		}

		return AllocationId;
	}

	int32 AddProfile(const UNeuralProfile* InProfile, const FGuid& InGuid, const FNeuralProfileStruct Settings)
	{
		check(InProfile);
		check(FindAllocationId(InProfile) == INDEX_NONE);

		int32 AllocationId = INDEX_NONE;
		{
			for (int i = 0; i < NeuralProfileEntries.Num(); ++i)
			{
				if (NeuralProfileEntries[i].Profile == nullptr)
				{
					AllocationId = i;
					FProfileEntry& Entry = NeuralProfileEntries[AllocationId];
					Entry.Profile = InProfile;

					break;
				}
			}

			if (AllocationId == INDEX_NONE)
			{
				AllocationId = NeuralProfileEntries.Num();
				FProfileEntry& Entry = NeuralProfileEntries.AddDefaulted_GetRef();
				Entry.Profile = InProfile;
			}
		}

		UpdateProfile(AllocationId, Settings);

		return AllocationId;
	}

	void UpdateProfile(int32 AllocationId, const FNeuralProfileStruct Settings)
	{
		check(IsInRenderingThread());
		
		if (AllocationId != INDEX_NONE && GNeuralProfileManager)
		{
			check(AllocationId < NeuralProfileEntries.Num());

			FNeuralProfileStruct& CurrentSettings = NeuralProfileEntries[AllocationId].Settings;
			const bool bModelIsTheSame =
				CurrentSettings.RuntimeType == Settings.RuntimeType &&
				CurrentSettings.NNEModelData == Settings.NNEModelData;

			const bool bNeedToUpdateModel = !bModelIsTheSame;
			
			// Update model
			if (bNeedToUpdateModel)
			{
				// Any changes to the setting require an update of the NNE model.
				GNeuralProfileManager->UpdateModel(AllocationId, Settings.NNEModelData.Get(), GetNeuralProfileRuntimeName(Settings.RuntimeType));
			}

			// Update tile setting
			if (CurrentSettings.TileSize != Settings.TileSize || bNeedToUpdateModel)
			{
				GNeuralProfileManager->UpdateTileType(AllocationId, Settings.TileSize);
			}

			if (CurrentSettings.TileOverlap != Settings.TileOverlap || bNeedToUpdateModel)
			{
				GNeuralProfileManager->UpdateTileOverlap(AllocationId, Settings.TileOverlap);
			}
			
			if (CurrentSettings.TileOverlapResolveType != Settings.TileOverlapResolveType || bNeedToUpdateModel)
			{
				GNeuralProfileManager->UpdateTileOverlapResolveType(AllocationId, Settings.TileOverlapResolveType);
			}

			// Update batch dimension
			const bool bBufferSizeNeedsUpdate =
				CurrentSettings.BatchSizeOverride != Settings.BatchSizeOverride;
			int32 BatchSizeOverride = NeuralProfileEntries[AllocationId].Settings.BatchSizeOverride;
			if (bBufferSizeNeedsUpdate/* || bNeedToUpdateModel*/)
			{
				if (GNeuralProfileManager->UpdateBatchSize(AllocationId, Settings.BatchSizeOverride))
				{
					BatchSizeOverride = Settings.BatchSizeOverride;
				}
			}

			NeuralProfileEntries[AllocationId].Settings = Settings;
			NeuralProfileEntries[AllocationId].Settings.BatchSizeOverride = BatchSizeOverride;

		}
	}

	void UpdateProfile(const UNeuralProfile* InProfile, const FNeuralProfileStruct Settings)
	{
		UpdateProfile(FindAllocationId(InProfile), Settings);
	}

	void RemoveProfile(const UNeuralProfile* InProfile)
	{
		int32 AllocationId = FindAllocationId(InProfile);

		if (AllocationId != INDEX_NONE)
		{
			check(AllocationId >= 0);
			check(NeuralProfileEntries[AllocationId].Profile == InProfile);
			FNeuralProfileStruct LocalSettings = NeuralProfileEntries[AllocationId].Settings;

			ENQUEUE_RENDER_COMMAND(RemoveNeuralProfile)(
				[LocalSettings, AllocationId](FRHICommandListImmediate& RHICmdList)
				{
					// any changes to the setting require an update of the NNE model.
					if (GNeuralProfileManager)
					{
						GNeuralProfileManager->RemoveModel(AllocationId);
					}
				});

			NeuralProfileEntries[AllocationId].Profile = nullptr;
			NeuralProfileEntries[AllocationId].Settings.Invalidate();
		}
	}

	int32 FindAllocationId(const UNeuralProfile* InProfile) const
	{
		for (int i = 0; i < NeuralProfileEntries.Num(); ++i)
		{
			if (NeuralProfileEntries[i].Profile == InProfile)
			{
				return i;
			}
		}

		return INDEX_NONE;
	}

	FNeuralProfileStruct GetProfileSetting(int32 AllocationId)
	{
		FNeuralProfileStruct Settings;
		if (AllocationId >= 0 && AllocationId < NeuralProfileEntries.Num())
		{
			Settings = NeuralProfileEntries[AllocationId].Settings;
		}

		return Settings;
	}

private:

	struct FProfileEntry
	{
		FNeuralProfileStruct Settings;
		const UNeuralProfile* Profile = nullptr;
	};

	TArray<FProfileEntry> NeuralProfileEntries;
};

// Global resources - lives on the render thread
TGlobalResource<FNeuralProfileModelTextureManager> GNeuralProfileModelTextureManager;

///////////////////////////////////////////////////////////////////////////////////////////////////
// UNeuralProfile

UNeuralProfile::UNeuralProfile(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

void UNeuralProfile::BeginDestroy()
{
	UNeuralProfile* Ref = this;
	ENQUEUE_RENDER_COMMAND(RemoveNeuralProfile)(
	[Ref](FRHICommandList& RHICmdList)
	{
		GNeuralProfileModelTextureManager.RemoveProfile(Ref);
	});

	Super::BeginDestroy();
}
#if WITH_EDITOR
void UNeuralProfile::PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);
	const FNeuralProfileStruct LocalSettings = this->Settings;
	UNeuralProfile* LocalProfile = this;
	GetRendererModule().InvalidatePathTracedOutput();

	ENQUEUE_RENDER_COMMAND(UpdateNeuralProfile)(
		[LocalSettings, LocalProfile](FRHICommandListImmediate& RHICmdList)
		{
			// any changes to the setting require an update of the texture
			GNeuralProfileModelTextureManager.UpdateProfile(LocalProfile, LocalSettings);
		});

	if (GNeuralProfileManager)
	{
		ENQUEUE_RENDER_COMMAND(UpdateDimensionParams)(
			[this](FRHICommandListImmediate& RHICmdList)
			{
				Settings.InputDimension = GNeuralProfileManager->GetInputDimension(
					Settings.NNEModelData.Get(),
					GetNeuralProfileRuntimeName(Settings.RuntimeType));

				Settings.OutputDimension = GNeuralProfileManager->GetOutputDimension(
					Settings.NNEModelData.Get(),
					GetNeuralProfileRuntimeName(Settings.RuntimeType));
			});
	}
}
#endif
/////////////////////////////////////////////////////////////////////////////////////////////////////
//// Public API
//
namespace NeuralProfile
{

	int32 GetNeuralProfileId(const UNeuralProfile* In)
	{
		int32 AllocationId = INDEX_NONE;
		if (In)
		{
			AllocationId = GNeuralProfileModelTextureManager.FindAllocationId(In);
		}

		return AllocationId;
	}

	int32 AddOrUpdateProfile(const UNeuralProfile* InProfile, const FGuid& InGuid, const FNeuralProfileStruct InSettings)
	{
		return GNeuralProfileModelTextureManager.AddOrUpdateProfile(InProfile, InGuid, InSettings);
	}

	::FNeuralProfileStruct GetProfileSetting(int32 AllocationId)
	{
		return GNeuralProfileModelTextureManager.GetProfileSetting(AllocationId);
	}

} // namespace NeuralProfile
