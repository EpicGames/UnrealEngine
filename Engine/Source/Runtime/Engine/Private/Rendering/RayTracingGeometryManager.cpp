// Copyright Epic Games, Inc. All Rights Reserved.

#include "Rendering/RayTracingGeometryManager.h"

#include "PrimitiveSceneProxy.h"
#include "SceneInterface.h"
#include "ComponentRecreateRenderStateContext.h"

#include "RHIResources.h"
#include "RHICommandList.h"

#include "RayTracingGeometry.h"
#include "RenderUtils.h"

#include "Rendering/RayTracingStreamableAsset.h"

#include "Math/UnitConversion.h"

#include "ProfilingDebugging/CsvProfiler.h"
#include "ProfilingDebugging/IoStoreTrace.h"

#include "Engine/Engine.h"

#if !UE_BUILD_SHIPPING
#include "HAL/FileManager.h"
#include "Misc/Paths.h"
#endif // !UE_BUILD_SHIPPING

#if RHI_RAYTRACING

DECLARE_LOG_CATEGORY_CLASS(LogRayTracingGeometryManager, Log, All);

static bool bHasRayTracingEnableChanged = false;
static TAutoConsoleVariable<int32> CVarRayTracingEnable(
	TEXT("r.RayTracing.Enable"),
	1,
	TEXT("Whether ray tracing is enabled at runtime.\n")
	TEXT("If r.RayTracing.EnableOnDemand is enabled, ray tracing can be toggled on/off at runtime. Otherwise this is only checked during initialization."),
	FConsoleVariableDelegate::CreateLambda([](IConsoleVariable* InVariable)
		{
			FGlobalComponentRecreateRenderStateContext Context;
			ENQUEUE_RENDER_COMMAND(RayTracingToggledCmd)(
				[](FRHICommandListImmediate&)
				{
					bHasRayTracingEnableChanged = true;
				}
			);
		}),
	ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<bool> CVarRayTracingUseReferenceBasedResidency(
	TEXT("r.RayTracing.UseReferenceBasedResidency"),
	true,
	TEXT("Whether raytracing geometries should be resident or evicted based on whether they're referenced in TLAS."),
	FConsoleVariableDelegate::CreateLambda([](IConsoleVariable* InVariable)
		{
			FGlobalComponentRecreateRenderStateContext Context;
			ENQUEUE_RENDER_COMMAND(RayTracingToggledCmd)(
				[](FRHICommandListImmediate&)
				{
					bHasRayTracingEnableChanged = true;
				}
			);
		}),
	ECVF_RenderThreadSafe
);

static int32 GRayTracingStreamingMaxPendingRequests = 128;
static FAutoConsoleVariableRef CVarNaniteStreamingMaxPendingRequests(
	TEXT("r.RayTracing.Streaming.MaxPendingRequests"),
	GRayTracingStreamingMaxPendingRequests,
	TEXT("Maximum number of requests that can be pending streaming."),
	ECVF_ReadOnly
);

static int32 GRayTracingResidentGeometryMemoryPoolSizeInMB = 400;
static FAutoConsoleVariableRef CVarRayTracingResidentGeometryMemoryPoolSizeInMB(
	TEXT("r.RayTracing.ResidentGeometryMemoryPoolSizeInMB"),
	GRayTracingResidentGeometryMemoryPoolSizeInMB,
	TEXT("Size of the ray tracing geometry pool.\n")
	TEXT("If pool size is larger than the requested geometry size, some unreferenced geometries will stay resident to reduce build overhead when they are requested again."),
	ECVF_RenderThreadSafe
);

static float GRayTracingApproximateCompactionRatio = 0.5f;
static FAutoConsoleVariableRef CVarRayTracingApproximateCompactionRatio(
	TEXT("r.RayTracing.ApproximateCompactionRatio"),
	GRayTracingApproximateCompactionRatio,
	TEXT("Ratio used by Ray Tracing Geometry Manager to approximate the ray tracing geometry size after compaction.\n")
	TEXT("This will be removed in a future version once Ray Tracing Geometry Manager tracks the actual compacted sizes."),
	ECVF_RenderThreadSafe
);

static bool bRefreshAlwaysResidentRayTracingGeometries = false;

static int32 GRayTracingNumAlwaysResidentLODs = 1;
static FAutoConsoleVariableRef CVarRayTracingNumAlwaysResidentLODs(
	TEXT("r.RayTracing.NumAlwaysResidentLODs"),
	GRayTracingNumAlwaysResidentLODs,
	TEXT("Number of LODs per ray tracing geometry group to always keep resident (even when not referenced by TLAS).\n")
	TEXT("Doesn't apply when ray tracing is disabled, in which case all ray tracing geometry is evicted."),
	FConsoleVariableDelegate::CreateLambda([](IConsoleVariable* InVariable)
		{
			ENQUEUE_RENDER_COMMAND(RefreshAlwaysResidentRayTracingGeometriesCmd)(
				[](FRHICommandListImmediate&)
				{
					bRefreshAlwaysResidentRayTracingGeometries = true;
				}
			);
		}),
	ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<int> CVarRayTracingAlwaysResidentWarningPercentage(
	TEXT("r.RayTracing.Debug.GeometryMemoryPool.AlwaysResidentWarningPercentage"),
	20,
	TEXT("A warning is triggered when always-resident size exceeds this percentage of the geometry memory pool [0, 100]."),
	ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<bool> CVarRayTracingOnDemandGeometryBuffersStreaming(
	TEXT("r.RayTracing.OnDemandGeometryBuffersStreaming"),
	true,
	TEXT("Whether to stream-in VB/IB buffers required to update dynamic geometry on-demand instead of keeping it in memory."),
	ECVF_RenderThreadSafe
);

static int32 GRayTracingMaxBuiltPrimitivesPerFrame = -1;
static FAutoConsoleVariableRef CVarRayTracingMaxBuiltPrimitivesPerFrame(
	TEXT("r.RayTracing.Geometry.MaxBuiltPrimitivesPerFrame"),
	GRayTracingMaxBuiltPrimitivesPerFrame,
	TEXT("Sets the ray tracing acceleration structure build budget in terms of maximum number of triangles per frame (<= 0 then disabled and all acceleration structures are build immediatly - default)"),
	ECVF_RenderThreadSafe
);

static float GRayTracingPendingBuildPriorityBoostPerFrame = 0.001f;
static FAutoConsoleVariableRef CVarRayTracingPendingBuildPriorityBoostPerFrame(
	TEXT("r.RayTracing.Geometry.PendingBuildPriorityBoostPerFrame"),
	GRayTracingPendingBuildPriorityBoostPerFrame,
	TEXT("Increment the priority for all pending build requests which are not scheduled that frame (0.001 - default)"),
	ECVF_RenderThreadSafe
);

static bool GRayTracingShowOnScreenWarnings = true;
static FAutoConsoleVariableRef CVarRayTracingShowOnScreenWarnings(
	TEXT("r.RayTracing.ShowOnScreenWarnings"),
	GRayTracingShowOnScreenWarnings,
	TEXT("Whether to show on-screen warnings related to ray tracing."),
	ECVF_RenderThreadSafe
);

#if DO_CHECK
static bool GRayTracingTestCheckIntegrity = false;
static FAutoConsoleVariableRef CVarRayTracingTestCheckIntegrity(
	TEXT("r.RayTracing.Test.CheckIntegrity"),
	GRayTracingTestCheckIntegrity,
	TEXT("Whether to check integrity of cached state related to ray tracing."),
	ECVF_RenderThreadSafe
);
#endif

#if !UE_BUILD_SHIPPING
static bool bDumpUnreferencedAlwaysResidentGeometriesNextFrame = false;

static FAutoConsoleCommand CmdDumpUnreferencedAlwaysResidentGeometries(
	TEXT("r.RayTracing.DumpUnreferencedAlwaysResidentGeometries"),
	TEXT("Writes the list of always-resident ray tracing geometries not currently referenced in the ray tracing scene to a CSV file"),
	FConsoleCommandDelegate::CreateStatic([] { bDumpUnreferencedAlwaysResidentGeometriesNextFrame = true; }),
	ECVF_Default);
#endif // !UE_BUILD_SHIPPING

DECLARE_STATS_GROUP(TEXT("Ray Tracing Geometry"), STATGROUP_RayTracingGeometry, STATCAT_Advanced);

DECLARE_DWORD_ACCUMULATOR_STAT(TEXT("Geometry Count"), STAT_RayTracingGeometryCount, STATGROUP_RayTracingGeometry);
DECLARE_DWORD_ACCUMULATOR_STAT(TEXT("Geometry Group Count"), STAT_RayTracingGeometryGroupCount, STATGROUP_RayTracingGeometry);

DECLARE_MEMORY_STAT(TEXT("Resident Memory"), STAT_RayTracingGeometryResidentMemory, STATGROUP_RayTracingGeometry);
DECLARE_MEMORY_STAT(TEXT("Always Resident Memory"), STAT_RayTracingGeometryAlwaysResidentMemory, STATGROUP_RayTracingGeometry);
DECLARE_MEMORY_STAT(TEXT("Referenced Memory"), STAT_RayTracingGeometryReferencedMemory, STATGROUP_RayTracingGeometry);
DECLARE_MEMORY_STAT(TEXT("Requested Memory"), STAT_RayTracingGeometryRequestedMemory, STATGROUP_RayTracingGeometry);

DECLARE_DWORD_ACCUMULATOR_STAT(TEXT("Pending Builds"), STAT_RayTracingPendingBuilds, STATGROUP_RayTracingGeometry);
DECLARE_DWORD_ACCUMULATOR_STAT(TEXT("Pending Build Primitives"), STAT_RayTracingPendingBuildPrimitives, STATGROUP_RayTracingGeometry);

DECLARE_DWORD_ACCUMULATOR_STAT(TEXT("Pending Streaming Requests"), STAT_RayTracingPendingStreamingRequests, STATGROUP_RayTracingGeometry);
DECLARE_DWORD_ACCUMULATOR_STAT(TEXT("In-flight Streaming Requests"), STAT_RayTracingInflightStreamingRequests, STATGROUP_RayTracingGeometry);

CSV_DEFINE_CATEGORY(RayTracingGeometry, true);

static bool IsAdditionalRayTracingGeometry(const FRayTracingGeometry* InGeometry)
{
	return InGeometry->LODIndex == FRayTracingGeometry::AdditionalGeometryLODIndex;
}

struct FReferenceBasedResidencyState
{
	struct FGeometryMetadata
	{
		uint64 LastReferencedFrame = 0;
		uint32 Size = 0;
		RayTracing::FGeometryGroupHandle GroupHandle;

		bool bAlwaysResident : 1 = false;
		bool bEvicted : 1 = false;
		bool bBVHStreamedOut : 1 = false;
		bool bBuffersStreamedOut : 1 = false;
	};

	struct FGeometryGroupMetadata
	{
		TArray<RayTracing::FGeometryHandle, TInlineAllocator<2>> GeometryHandles;
		TSet<RayTracing::FGeometryHandle> AdditionalGeometryHandles;
		int32 CurrentFirstLODIdx = INDEX_NONE;
	};

	FGeometryMetadata& GetGeometryMetadata(RayTracing::FGeometryHandle Handle);
	FGeometryGroupMetadata& GetGroupMetadata(RayTracing::FGeometryGroupHandle Handle);

	void Update();

	uint64 FrameCounter = ~0ULL;

	int64 TotalResidentSize = 0;
	int64 TotalAlwaysResidentSize = 0;
	int64 TotalStreamingSize = 0;

	TSparseArray<FGeometryMetadata> GeometryMetadatas;
	TSparseArray<FGeometryGroupMetadata> GroupMetadatas;

	TSet<RayTracing::FGeometryHandle> EvictableGeometries;

	TSet<RayTracing::FGeometryHandle> ReferencedGeometries;
	TSet<RayTracing::FGeometryGroupHandle> ReferencedGeometryGroups;
	TSet<RayTracing::FGeometryGroupHandle> ReferencedGeometryGroupsForDynamicUpdate;

	// Update output

	TArray<RayTracing::FGeometryHandle> GeometriesToEvict;
	TArray<RayTracing::FGeometryHandle> GeometriesToMakeResidentAndStreamIn;

	int64 ReferencedSize = 0;
	int64 RequestedSize = 0;
	int64 RequestedButEvictedSize = 0;
};

FReferenceBasedResidencyState::FGeometryMetadata& FReferenceBasedResidencyState::GetGeometryMetadata(RayTracing::FGeometryHandle Handle)
{
	checkf(!Handle.IsNull(), TEXT("Must provide valid RayTracing::FGeometryHandle."));

	const uint32 GeometryIndex = FRayTracingGeometryManager::GetHandleIndex(Handle);

	checkf(GeometryMetadatas.IsValidIndex(GeometryIndex), TEXT("Stale RayTracing::FGeometryHandle detected - invalid index %u."), GeometryIndex);

	return GeometryMetadatas[GeometryIndex];
}

FReferenceBasedResidencyState::FGeometryGroupMetadata& FReferenceBasedResidencyState::GetGroupMetadata(RayTracing::FGeometryGroupHandle Handle)
{
	checkf(!Handle.IsNull(), TEXT("Must provide valid RayTracing::FGeometryGroupHandle."));

	const uint32 GroupIndex = FRayTracingGeometryManager::GetHandleIndex(Handle);

	checkf(GroupMetadatas.IsValidIndex(GroupIndex), TEXT("Stale RayTracing::FGeometryGroupHandle detected - invalid index %u."), GroupIndex);

	return GroupMetadatas[GroupIndex];
}

void FReferenceBasedResidencyState::Update()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FReferenceBasedResidencyState::Update);

	// TODO: Should also add support for evicting resident VB/IB buffers when no longer referenced in ReferencedGeometryGroupsForDynamicUpdate
	// Need a separate set of "EvictableBuffers" set and logic to handle it.

	TSet<RayTracing::FGeometryHandle> NotReferencedResidentGeometries = EvictableGeometries;

	GeometriesToMakeResidentAndStreamIn.Reserve(GeometryMetadatas.Num());

	ReferencedSize = 0;
	RequestedSize = TotalAlwaysResidentSize;
	RequestedButEvictedSize = 0;

	// Step 1
	// - update LastReferencedFrame of referenced geometries and calculate memory required to make evicted geometries resident
	for (RayTracing::FGeometryHandle GeometryHandle : ReferencedGeometries)
	{
		FGeometryMetadata& GeometryMetadata = GetGeometryMetadata(GeometryHandle);
		GeometryMetadata.LastReferencedFrame = FrameCounter;

		if (GeometryMetadata.bEvicted || GeometryMetadata.bBVHStreamedOut || (GeometryMetadata.bBuffersStreamedOut && ReferencedGeometryGroupsForDynamicUpdate.Contains(GeometryMetadata.GroupHandle)))
		{
			GeometriesToMakeResidentAndStreamIn.Add(GeometryHandle);
		}

		NotReferencedResidentGeometries.Remove(GeometryHandle);

		ReferencedSize += GeometryMetadata.Size;

		if (!GeometryMetadata.bAlwaysResident)
		{
			RequestedSize += GeometryMetadata.Size;
		}

		if (GeometryMetadata.bEvicted)
		{
			RequestedButEvictedSize += GeometryMetadata.Size;
		}
	}

	// Step 2
	// - add all geometries in referenced groups to ReferencedGeometriesEvictedOrRequiringStreaming
	//		- need to make all geometries in group resident otherwise might not have valid geometry when reducing LOD
	//		- TODO: Could track TargetLOD and only make [TargetLOD ... LastLOD] range resident
	// - also update LastReferencedFrame and calculate memory required to make evicted geometries resident
	for (RayTracing::FGeometryGroupHandle GroupHandle : ReferencedGeometryGroups)
	{
		const FGeometryGroupMetadata& GroupMetadata = GetGroupMetadata(GroupHandle);
		const int32 NumGeometriesInGroup = GroupMetadata.GeometryHandles.Num();

		for (int32 LODIndex = GroupMetadata.CurrentFirstLODIdx; LODIndex < NumGeometriesInGroup; ++LODIndex)
		{
			RayTracing::FGeometryHandle GeometryHandle = GroupMetadata.GeometryHandles[LODIndex];

			if (!GeometryHandle.IsNull()) // some LODs might be stripped during cook
			{
				FGeometryMetadata& GeometryMetadata = GetGeometryMetadata(GeometryHandle);
				GeometryMetadata.LastReferencedFrame = FrameCounter;

				ReferencedSize += GeometryMetadata.Size;

				if (GeometryMetadata.bAlwaysResident)
				{
					checkf(!GeometryMetadata.bEvicted, TEXT("Always-resident ray tracing geometry was unexpectedly evicted."));
				}
				else
				{
					if (GeometryMetadata.bEvicted || GeometryMetadata.bBVHStreamedOut || (GeometryMetadata.bBuffersStreamedOut && ReferencedGeometryGroupsForDynamicUpdate.Contains(GeometryMetadata.GroupHandle)))
					{
						GeometriesToMakeResidentAndStreamIn.Add(GeometryHandle);
					}

					NotReferencedResidentGeometries.Remove(GeometryHandle);

					RequestedSize += GeometryMetadata.Size;

					if (GeometryMetadata.bEvicted)
					{
						RequestedButEvictedSize += GeometryMetadata.Size;
					}
				}
			}
		}

		for (RayTracing::FGeometryHandle GeometryHandle : GroupMetadata.AdditionalGeometryHandles)
		{
			FGeometryMetadata& GeometryMetadata = GetGeometryMetadata(GeometryHandle);
			GeometryMetadata.LastReferencedFrame = FrameCounter;

			ReferencedSize += GeometryMetadata.Size;

			checkf(!GeometryMetadata.bAlwaysResident, TEXT("Additional geometries can't be always-resident."));

			if (GeometryMetadata.bEvicted || GeometryMetadata.bBVHStreamedOut || (GeometryMetadata.bBuffersStreamedOut && ReferencedGeometryGroupsForDynamicUpdate.Contains(GeometryMetadata.GroupHandle)))
			{
				GeometriesToMakeResidentAndStreamIn.Add(GeometryHandle);
			}

			NotReferencedResidentGeometries.Remove(GeometryHandle);

			RequestedSize += GeometryMetadata.Size;

			if (GeometryMetadata.bEvicted)
			{
				RequestedButEvictedSize += GeometryMetadata.Size;
			}
		}
	}

#if DO_CHECK
	// ensure(GeometriesToMakeResidentAndStreamIn.Num() == TSet(GeometriesToMakeResidentAndStreamIn).Num());
#endif

	const int64 ResidentGeometryMemoryPoolSize = FUnitConversion::Convert(GRayTracingResidentGeometryMemoryPoolSizeInMB, EUnit::Megabytes, EUnit::Bytes);

	// Step 3
	// - if making requested geometries resident will put us over budget -> evict some geometry not referenced by TLAS
	if (TotalResidentSize + TotalStreamingSize + RequestedButEvictedSize > ResidentGeometryMemoryPoolSize)
	{
		GeometriesToEvict = NotReferencedResidentGeometries.Array();

		// sort to evict geometries in the following order:
		//	- least recently used
		//	- largest geometries
		Algo::Sort(GeometriesToEvict, [this](RayTracing::FGeometryHandle& LHSHandle, RayTracing::FGeometryHandle& RHSHandle)
			{
				FGeometryMetadata& LHS = GetGeometryMetadata(LHSHandle);
				FGeometryMetadata& RHS = GetGeometryMetadata(RHSHandle);

				// TODO: evict unreferenced dynamic geometries using shared buffers first since they need to be rebuild anyway
				// (and then dynamic geometries requiring update?

				// 1st - last referenced frame
				if (LHS.LastReferencedFrame != RHS.LastReferencedFrame)
				{
					return LHS.LastReferencedFrame < RHS.LastReferencedFrame;
				}

				// 2nd - size
				return LHS.Size > RHS.Size;
			});
	}
	else
	{
		GeometriesToEvict.Empty();
	}

	// Step 4
	// - make referenced geometries resident until we go over budget
	if (TotalResidentSize + TotalStreamingSize < ResidentGeometryMemoryPoolSize)
	{
		// sort by size to prioritize smaller geometries
		Algo::Sort(GeometriesToMakeResidentAndStreamIn, [this](RayTracing::FGeometryHandle& LHSHandle, RayTracing::FGeometryHandle& RHSHandle)
			{
				FGeometryMetadata& LHS = GetGeometryMetadata(LHSHandle);
				FGeometryMetadata& RHS = GetGeometryMetadata(RHSHandle);

				return LHS.Size < RHS.Size;
			});
	}
}

FRayTracingGeometryManager::FRayTracingGeometryManager()
{
	StreamingRequests.SetNum(GRayTracingStreamingMaxPendingRequests);

	ReferenceBasedResidencyState = MakePimpl<FReferenceBasedResidencyState>();

#if CSV_PROFILER_STATS
	// Need to defer until RHI is initialized.
	FCoreDelegates::GetOnPostEngineInit().AddLambda([]
	{
		CSV_METADATA(TEXT("RayTracing"), IsRayTracingEnabled() ? TEXT("1") : TEXT("0"));
	});
#endif
}

FRayTracingGeometryManager::~FRayTracingGeometryManager()
{
	ensure(GeometryBuildRequests.IsEmpty());
	ensure(RegisteredGeometries.IsEmpty());
	ensure(RegisteredGroups.IsEmpty());

	ReferenceBasedResidencyTask.Wait();
	ReferenceBasedResidencyTask = {};
}

FRayTracingGeometryManager::FBuildRequest& FRayTracingGeometryManager::GetBuildRequest(RayTracing::FBuildRequestHandle Handle)
{
	checkf(!Handle.IsNull(), TEXT("Must provide valid RayTracing::FBuildRequestHandle."));

	const uint32 RequestIndex = GetHandleIndex(Handle);
	const uint32 RequestVersion = GetHandleVersion(Handle);

	checkf(GeometryBuildRequests.IsValidIndex(RequestIndex), TEXT("Stale RayTracing::FBuildRequestHandle detected - invalid index %u."), RequestIndex);
	checkf(RequestVersion == GeometryBuildRequestsVersions[RequestIndex], TEXT("Stale RayTracing::FBuildRequestHandle detected - invalid version %u (expected %u)."), RequestVersion, GeometryBuildRequestsVersions[RequestIndex]);

	return GeometryBuildRequests[RequestIndex];
}

bool FRayTracingGeometryManager::IsHandleValid(RayTracing::FGeometryHandle Handle) const
{
	const uint32 GeometryIndex = GetHandleIndex(Handle);
	const uint32 GeometryVersion = GetHandleVersion(Handle);

	return RegisteredGeometries.IsValidIndex(GeometryIndex) && (GeometryVersion == RegisteredGeometriesVersions[GeometryIndex]);
}

bool FRayTracingGeometryManager::IsHandleValid(RayTracing::FGeometryGroupHandle Handle) const
{
	const uint32 GroupIndex = GetHandleIndex(Handle);
	const uint32 GroupVersion = GetHandleVersion(Handle);

	return RegisteredGroups.IsValidIndex(GroupIndex) && (GroupVersion == RegisteredGroupsVersions[GroupIndex]);
}

uint32 FRayTracingGeometryManager::GetRegisteredGeometryIndex(RayTracing::FGeometryHandle Handle) const
{
	checkf(!Handle.IsNull(), TEXT("Must provide valid RayTracing::FGeometryHandle."));

	const uint32 GeometryIndex = GetHandleIndex(Handle);
	const uint32 GeometryVersion = GetHandleVersion(Handle);

	checkf(RegisteredGeometries.IsValidIndex(GeometryIndex), TEXT("Stale RayTracing::FGeometryHandle detected - invalid index %u."), GeometryIndex);
	checkf(GeometryVersion == RegisteredGeometriesVersions[GeometryIndex], TEXT("Stale RayTracing::FGeometryHandle detected - invalid version %u (expected %u)."), GeometryVersion, RegisteredGeometriesVersions[GeometryIndex]);

	return GeometryIndex;
}

uint32 FRayTracingGeometryManager::GetRegisteredGroupIndex(RayTracing::FGeometryGroupHandle Handle) const
{
	checkf(!Handle.IsNull(), TEXT("Must provide valid RayTracing::FGeometryGroupHandle."));

	const uint32 GroupIndex = GetHandleIndex(Handle);
	const uint32 GroupVersion = GetHandleVersion(Handle);

	checkf(RegisteredGroups.IsValidIndex(GroupIndex), TEXT("Stale RayTracing::FGeometryGroupHandle detected - invalid index %u."), GroupIndex);
	checkf(GroupVersion == RegisteredGroupsVersions[GroupIndex], TEXT("Stale RayTracing::FGeometryGroupHandle detected - invalid version %u (expected %u)."), GroupVersion, RegisteredGroupsVersions[GroupIndex]);

	return GroupIndex;
}

FRayTracingGeometryManager::FRegisteredGeometry& FRayTracingGeometryManager::GetRegisteredGeometry(RayTracing::FGeometryHandle Handle)
{
	return RegisteredGeometries[GetRegisteredGeometryIndex(Handle)];
}

const FRayTracingGeometryManager::FRegisteredGeometry& FRayTracingGeometryManager::GetRegisteredGeometry(RayTracing::FGeometryHandle Handle) const
{
	return RegisteredGeometries[GetRegisteredGeometryIndex(Handle)];
}

FRayTracingGeometryManager::FRayTracingGeometryGroup& FRayTracingGeometryManager::GetRegisteredGroup(RayTracing::FGeometryGroupHandle Handle)
{
	return RegisteredGroups[GetRegisteredGroupIndex(Handle)];
}

const FRayTracingGeometryManager::FRayTracingGeometryGroup& FRayTracingGeometryManager::GetRegisteredGroup(RayTracing::FGeometryGroupHandle Handle) const
{
	return RegisteredGroups[GetRegisteredGroupIndex(Handle)];
}

static float GetInitialBuildPriority(ERTAccelerationStructureBuildPriority InBuildPriority)
{
	switch (InBuildPriority)
	{
	case ERTAccelerationStructureBuildPriority::Immediate:	return 1.0f;
	case ERTAccelerationStructureBuildPriority::High:		return 0.5f;
	case ERTAccelerationStructureBuildPriority::Normal:		return 0.24f;
	case ERTAccelerationStructureBuildPriority::Low:		return 0.01f;
	case ERTAccelerationStructureBuildPriority::Skip:
	default:
	{
		checkNoEntry();
		return 0.0f;
	}
	}
}

RayTracing::FBuildRequestHandle FRayTracingGeometryManager::RequestBuildAccelerationStructure(FRayTracingGeometry* InGeometry, ERTAccelerationStructureBuildPriority InPriority, EAccelerationStructureBuildMode InBuildMode)
{
	check(InGeometry->RayTracingBuildRequestIndex.IsNull());

	FBuildRequest Request;
	Request.BuildPriority = GetInitialBuildPriority(InPriority);
	Request.Owner = InGeometry;
	Request.BuildMode = EAccelerationStructureBuildMode::Build;

	FScopeLock ScopeLock(&RequestCS);

	const uint32 RequestIndex = GeometryBuildRequests.Add(Request);

	GeometryBuildRequestsVersions.PadToNum(GeometryBuildRequests.GetMaxIndex(), 0);

	const uint32 RequestVersion = GeometryBuildRequestsVersions[RequestIndex];

	const RayTracing::FBuildRequestHandle RequestHandle = MakeBuildRequestHandle(RequestVersion, RequestIndex);
	
	GeometryBuildRequests[RequestIndex].RequestHandle = RequestHandle;

	InGeometry->RayTracingBuildRequestIndex = RequestHandle;

	INC_DWORD_STAT(STAT_RayTracingPendingBuilds);
	INC_DWORD_STAT_BY(STAT_RayTracingPendingBuildPrimitives, InGeometry->GetInitializer().TotalPrimitiveCount);
	
	return RequestHandle;
}

void FRayTracingGeometryManager::RemoveBuildRequest(RayTracing::FBuildRequestHandle InRequestHandle)
{
	FScopeLock ScopeLock(&RequestCS);

	FBuildRequest& BuildRequest = GetBuildRequest(InRequestHandle);

	DEC_DWORD_STAT(STAT_RayTracingPendingBuilds);
	DEC_DWORD_STAT_BY(STAT_RayTracingPendingBuildPrimitives, BuildRequest.Owner->GetInitializer().TotalPrimitiveCount);

	BuildRequest.Owner->RayTracingBuildRequestIndex = {};

	const uint32 RequestIndex = GetHandleIndex(InRequestHandle);
	GeometryBuildRequests.RemoveAt(RequestIndex);
	GeometryBuildRequestsVersions[RequestIndex] = (GeometryBuildRequestsVersions[RequestIndex] + 1) & HandleVersionMask;
}

bool FRayTracingGeometryManager::IsAlwaysResidentGeometry(const FRayTracingGeometry* InGeometry, const FRayTracingGeometryGroup& Group)
{
	return !IsAdditionalRayTracingGeometry(InGeometry) && InGeometry->LODIndex >= Group.GeometryHandles.Num() - GRayTracingNumAlwaysResidentLODs;
}

RayTracing::FGeometryGroupHandle FRayTracingGeometryManager::RegisterRayTracingGeometryGroup(uint32 NumLODs, uint32 CurrentFirstLODIdx)
{
	FScopeLock ScopeLock(&MainCS);

	FRayTracingGeometryGroup Group;
	Group.GeometryHandles.Init({}, NumLODs);
	Group.NumReferences = 1;
	Group.CurrentFirstLODIdx = CurrentFirstLODIdx;

	const uint32 GroupIndex = RegisteredGroups.Add(MoveTemp(Group));

	RegisteredGroupsVersions.PadToNum(RegisteredGroups.GetMaxIndex(), 0);

	const uint32 GroupVersion = RegisteredGroupsVersions[GroupIndex];

	const RayTracing::FGeometryGroupHandle Handle = MakeGeometryGroupHandle(GroupVersion, GroupIndex);

	INC_DWORD_STAT(STAT_RayTracingGeometryGroupCount);

	ModifiedGroups.Add(Handle);

	return Handle;
}

void FRayTracingGeometryManager::ReleaseRayTracingGeometryGroup(RayTracing::FGeometryGroupHandle Handle)
{
	FScopeLock ScopeLock(&MainCS);

	ReleaseRayTracingGeometryGroupReference(Handle);
}

void FRayTracingGeometryManager::ReleaseRayTracingGeometryGroupReference(RayTracing::FGeometryGroupHandle Handle)
{
	FRayTracingGeometryGroup& Group = GetRegisteredGroup(Handle);

	--Group.NumReferences;

	if (Group.NumReferences == 0)
	{
		for (RayTracing::FGeometryHandle GeometryHandle : Group.GeometryHandles)
		{
			checkf(GeometryHandle.IsNull(), TEXT("All FRayTracingGeometry in a group must be unregistered before releasing the group."));
		}

		checkf(Group.AdditionalGeometryHandles.IsEmpty(), TEXT("All Additional FRayTracingGeometry in a group must be unregistered before releasing the group."));

		check(Group.ProxiesWithCachedRayTracingState.IsEmpty());

		const uint32 GroupIndex = GetHandleIndex(Handle);
		RegisteredGroups.RemoveAt(GroupIndex);
		RegisteredGroupsVersions[GroupIndex] = (RegisteredGroupsVersions[GroupIndex] + 1) & HandleVersionMask;

		ReferencedGeometryGroups.Remove(Handle);
		ReferencedGeometryGroupsForDynamicUpdate.Remove(Handle);

		DEC_DWORD_STAT(STAT_RayTracingGeometryGroupCount);

		ReleasedGroups.Add(Handle);
		ModifiedGroups.Remove(Handle);
	}
	else
	{
		ModifiedGroups.Add(Handle);
	}
}

RayTracing::FGeometryHandle FRayTracingGeometryManager::RegisterRayTracingGeometry(FRayTracingGeometry* InGeometry)
{
	check(InGeometry);

	FScopeLock ScopeLock(&MainCS);

	const uint32 GeometryIndex = RegisteredGeometries.Add({});

	RegisteredGeometriesVersions.PadToNum(RegisteredGeometries.GetMaxIndex(), 0);

	const uint32 GeometryVersion = RegisteredGeometriesVersions[GeometryIndex];

	const RayTracing::FGeometryHandle Handle = MakeGeometryHandle(GeometryVersion, GeometryIndex);

	FRegisteredGeometry& RegisteredGeometry = RegisteredGeometries[GeometryIndex];
	RegisteredGeometry.Geometry = InGeometry;

	const RayTracing::FGeometryGroupHandle GroupHandle = InGeometry->GroupHandle;
	const int8 LODIndex = InGeometry->LODIndex;

	if (!GroupHandle.IsNull())
	{
		FRayTracingGeometryGroup& Group = GetRegisteredGroup(GroupHandle);

		const bool bIsAdditionalRayTracingGeometry = IsAdditionalRayTracingGeometry(InGeometry);

		if (!bIsAdditionalRayTracingGeometry)
		{
			checkf(LODIndex >= 0 && LODIndex < Group.GeometryHandles.Num(), TEXT("FRayTracingGeometry assigned to a group must have a valid LODIndex"));
			checkf(Group.GeometryHandles[LODIndex].IsNull(), TEXT("Each LOD inside a FRayTracingGeometryGroup can only be associated with a single FRayTracingGeometry"));

			Group.GeometryHandles[LODIndex] = Handle;
		}
		else
		{
			check(!Group.AdditionalGeometryHandles.Contains(Handle));
			Group.AdditionalGeometryHandles.Add(Handle);
		}

		++Group.NumReferences;

		ModifiedGroups.Add(GroupHandle);

		RegisteredGeometry.bAlwaysResident = IsAlwaysResidentGeometry(InGeometry, Group);

		if (RegisteredGeometry.bAlwaysResident)
		{
			AlwaysResidentGeometries.Add(Handle);
		}

		if (IsRayTracingEnabled() && (bIsAdditionalRayTracingGeometry || LODIndex >= Group.CurrentFirstLODIdx) && (!IsRayTracingUsingReferenceBasedResidency() || RegisteredGeometry.bAlwaysResident))
		{
			// TODO: Should also do this if the geometry is not assigned to a group?
			PendingStreamingRequests.Add(Handle);
			INC_DWORD_STAT(STAT_RayTracingPendingStreamingRequests);
		}
	}
		
	INC_DWORD_STAT(STAT_RayTracingGeometryCount);

	RefreshRegisteredGeometry(Handle);

	return Handle;
}

void FRayTracingGeometryManager::ReleaseRayTracingGeometryHandle(RayTracing::FGeometryHandle Handle)
{
	FScopeLock ScopeLock(&MainCS);

	FRegisteredGeometry& RegisteredGeometry = GetRegisteredGeometry(Handle);

	// Cancel associated streaming request if currently in-flight
	CancelStreamingRequest(RegisteredGeometry);

	if (!RegisteredGeometry.Geometry->GroupHandle.IsNull())
	{
		// if geometry was assigned to a group, clear the relevant entry so another geometry can be registered later

		FRayTracingGeometryGroup& Group = GetRegisteredGroup(RegisteredGeometry.Geometry->GroupHandle);

		if (!IsAdditionalRayTracingGeometry(RegisteredGeometry.Geometry))
		{
			checkf(RegisteredGeometry.Geometry->LODIndex >= 0 && RegisteredGeometry.Geometry->LODIndex < Group.GeometryHandles.Num(), TEXT("FRayTracingGeometry assigned to a group must have a valid LODIndex"));
			checkf(Group.GeometryHandles[RegisteredGeometry.Geometry->LODIndex] == Handle, TEXT("Unexpected mismatch of FRayTracingGeometry in FRayTracingGeometryGroup"));

			Group.GeometryHandles[RegisteredGeometry.Geometry->LODIndex] = {};
		}
		else
		{
			check(Group.AdditionalGeometryHandles.Contains(Handle));
			Group.AdditionalGeometryHandles.Remove(Handle);
		}

		ReleaseRayTracingGeometryGroupReference(RegisteredGeometry.Geometry->GroupHandle);
	}

	{
		int32 NumRemoved = ResidentGeometries.Remove(Handle);

		if (NumRemoved > 0)
		{
			TotalResidentSize -= RegisteredGeometry.Size;
		}
	}

	{
		int32 NumRemoved = AlwaysResidentGeometries.Remove(Handle);

		if (NumRemoved > 0)
		{
			checkf(RegisteredGeometry.bAlwaysResident, TEXT("Geometry should have the bAlwaysResident flag enabled since it was in the AlwaysResidentGeometries set."));

			TotalAlwaysResidentSize -= RegisteredGeometry.Size;
		}
	}

	const uint32 GeometryIndex = GetHandleIndex(Handle);
	RegisteredGeometries.RemoveAt(GeometryIndex);
	RegisteredGeometriesVersions[GeometryIndex] = (RegisteredGeometriesVersions[GeometryIndex] + 1) & HandleVersionMask;

	ReferencedGeometries.Remove(Handle);
	if (PendingStreamingRequests.Remove(Handle) > 0)
	{
		DEC_DWORD_STAT(STAT_RayTracingPendingStreamingRequests);
	}

	DEC_DWORD_STAT(STAT_RayTracingGeometryCount);

	ReleasedGeometries.Add(Handle);
	ModifiedGeometries.Remove(Handle);
}

void FRayTracingGeometryManager::SetRayTracingGeometryStreamingData(const FRayTracingGeometry* Geometry, FRayTracingStreamableAsset& StreamableAsset)
{
	FScopeLock ScopeLock(&MainCS);

	const RayTracing::FGeometryHandle GeometryHandle = Geometry->RayTracingGeometryHandle;

	FRegisteredGeometry& RegisteredGeometry = GetRegisteredGeometry(GeometryHandle);
	RegisteredGeometry.StreamableAsset = &StreamableAsset;
	RegisteredGeometry.StreamableBVHSize = StreamableAsset.GetRequestSizeBVH();
	RegisteredGeometry.StreamableBuffersSize = StreamableAsset.GetRequestSizeBuffers();

	checkf(RegisteredGeometry.StreamableBVHSize > 0 || RegisteredGeometry.StreamableBuffersSize > 0, TEXT("FRayTracingStreamableAsset should have data to stream."));
}

void FRayTracingGeometryManager::SetRayTracingGeometryGroupCurrentFirstLODIndex(FRHICommandListBase& RHICmdList, RayTracing::FGeometryGroupHandle Handle, uint8 NewCurrentFirstLODIdx)
{
	FScopeLock ScopeLock(&MainCS);

	FRayTracingGeometryGroup& Group = GetRegisteredGroup(Handle);

	// immediately release streamed out LODs
	if (NewCurrentFirstLODIdx > Group.CurrentFirstLODIdx)
	{
		FRHIResourceReplaceBatcher Batcher(RHICmdList, NewCurrentFirstLODIdx - Group.CurrentFirstLODIdx);
		for (int32 LODIdx = Group.CurrentFirstLODIdx; LODIdx < NewCurrentFirstLODIdx; ++LODIdx)
		{
			RayTracing::FGeometryHandle GeometryHandle = Group.GeometryHandles[LODIdx];

			// some LODs might be stripped during cook
			// skeletal meshes only create static LOD when rendering as static
			if (GeometryHandle.IsNull())
			{
				continue;
			}

			FRegisteredGeometry& RegisteredGeometry = GetRegisteredGeometry(GeometryHandle);

			if (!RegisteredGeometry.Geometry->IsEvicted())
			{
				// Cancel associated streaming request if currently in-flight
				CancelStreamingRequest(RegisteredGeometry);

				StreamOutGeometry(Batcher, RegisteredGeometry);
			}
		}
	}
	else if (IsRayTracingEnabled() && !IsRayTracingUsingReferenceBasedResidency())
	{
		for (int32 LODIdx = NewCurrentFirstLODIdx; LODIdx < Group.CurrentFirstLODIdx; ++LODIdx)
		{
			if (!Group.GeometryHandles[LODIdx].IsNull())
			{
				// TODO: should do this for always-resident mips even when using reference based residency
				PendingStreamingRequests.Add(Group.GeometryHandles[LODIdx]);
				INC_DWORD_STAT(STAT_RayTracingPendingStreamingRequests);
			}
		}
	}

	Group.CurrentFirstLODIdx = NewCurrentFirstLODIdx;

	ModifiedGroups.Add(Handle);
}

static bool ShouldCompactAfterBuild(const FRayTracingGeometryInitializer& Initializer)
{
	return Initializer.bAllowCompaction
		&& !Initializer.bFastBuild
		&& !Initializer.bAllowUpdate
		&& !Initializer.OfflineDataHeader.IsValid()
		&& GRHIGlobals.RayTracing.SupportsAccelerationStructureCompaction;
}

void FRayTracingGeometryManager::RefreshRegisteredGeometry(RayTracing::FGeometryHandle Handle)
{
	FScopeLock ScopeLock(&MainCS);

	if (!Handle.IsNull())
	{
		FRegisteredGeometry& RegisteredGeometry = GetRegisteredGeometry(Handle);
		
		const FRayTracingGeometryInitializer& GeometryInitializer = RegisteredGeometry.Geometry->GetInitializer();

		const uint32 OldSize = RegisteredGeometry.Size;

		// Update size - Geometry RHI might not be valid yet (evicted or uninitialized), so calculate size using Initializer here
		{
			// TODO: Should call RegisteredGeometry.Geometry->HasValidInitializer() instead?
			bool bAllSegmentsAreValid = GeometryInitializer.Segments.Num() > 0;
			for (const FRayTracingGeometrySegment& Segment : GeometryInitializer.Segments)
			{
				if (!Segment.VertexBuffer)
				{
					bAllSegmentsAreValid = false;
					break;
				}
			}

			if (bAllSegmentsAreValid)
			{
				RegisteredGeometry.Size = RHICalcRayTracingGeometrySize(GeometryInitializer).ResultSize;

				if (ShouldCompactAfterBuild(GeometryInitializer))
				{
					RegisteredGeometry.Size = uint32(RegisteredGeometry.Size * GRayTracingApproximateCompactionRatio);
				}
			}
			else
			{
				RegisteredGeometry.Size = 0;
			}
		}

		RegisteredGeometry.bEvicted = RegisteredGeometry.Geometry->IsEvicted();

		if (RegisteredGeometry.bAlwaysResident)
		{
			checkf(AlwaysResidentGeometries.Contains(Handle), TEXT("Geometry with bAlwaysResident flag set should be in the AlwaysResidentGeometries set."));

			TotalAlwaysResidentSize -= OldSize;
			TotalAlwaysResidentSize += RegisteredGeometry.Size;
		}

		if (RegisteredGeometry.Geometry->IsValid() && !RegisteredGeometry.bEvicted)
		{
			bool bAlreadyInSet;
			ResidentGeometries.Add(Handle, &bAlreadyInSet);

			if (bAlreadyInSet)
			{
				TotalResidentSize -= OldSize;
			}

			TotalResidentSize += RegisteredGeometry.Size;
		}
		else
		{
			int32 NumRemoved = ResidentGeometries.Remove(Handle);

			if (NumRemoved > 0)
			{
				TotalResidentSize -= OldSize;
			}
		}

		checkf(!AlwaysResidentGeometries.Contains(Handle) || !RegisteredGeometry.bEvicted || !IsRayTracingEnabled(), TEXT("Always-resident geometries can't be evicted"));

		ModifiedGeometries.Add(Handle);
	}
}

void FRayTracingGeometryManager::PreRender()
{
	bRenderedFrame = true;
}

void FRayTracingGeometryManager::Update(FRHICommandList& RHICmdList)
{
	ReferenceBasedResidencyTask.Wait();
	ReferenceBasedResidencyTask = {};

	ProcessReferenceBasedResidency(RHICmdList);
}

void FRayTracingGeometryManager::Tick(FRHICommandList& RHICmdList)
{
	if (IsRunningCommandlet())
	{
		return;
	}

	check(IsInRenderingThread());

	TRACE_CPUPROFILER_EVENT_SCOPE(FRayTracingGeometryManager::Tick);
	QUICK_SCOPE_CYCLE_COUNTER(STAT_FRayTracingGeometryManager_Tick);
	CSV_SCOPED_TIMING_STAT_EXCLUSIVE(RayTracingGeometryManager_Tick);

	ReferenceBasedResidencyTask.Wait();
	ReferenceBasedResidencyTask = {};

	// TODO: investigate fine grained locking to minimize blocking progress on render command pipes
	// - Don't touch registered geometry/group arrays from render command pipes
	//   - Separate arrays of free geometry/group handles + HandleAllocationCS
	//   - delay actual registration until PreRender() which happens on Render Thread
	//	 - Tick() doesn't need to lock at all
	// - Refresh requests could be queued and processed during Tick()
	FScopeLock ScopeLock(&MainCS);

#if DO_CHECK
	static uint64 PreviousFrameCounter = GFrameCounterRenderThread - 1;
	checkf(GFrameCounterRenderThread != PreviousFrameCounter, TEXT("FRayTracingGeometryManager::Tick() should only be called once per frame"));
	PreviousFrameCounter = GFrameCounterRenderThread;
#endif

	const bool bIsRayTracingEnabled = IsRayTracingEnabled();
	const bool bUsingReferenceBasedResidency = IsRayTracingUsingReferenceBasedResidency();

	checkf(bUsingReferenceBasedResidency || (ReferencedGeometries.IsEmpty() && ReferencedGeometryGroups.IsEmpty() && ReferencedGeometryGroupsForDynamicUpdate.IsEmpty()),
		TEXT("ReferencedGeometryHandles, ReferencedGeometryGroups and ReferencedGeometryGroupsForDynamicUpdate are expected to be empty when not using reference based residency"));

	RefreshAlwaysResidentRayTracingGeometries(RHICmdList);

	UpdateReferenceBasedResidencyState();

#if DO_CHECK
	CheckIntegrity();
#endif

	if (!bIsRayTracingEnabled)
	{
		if (bHasRayTracingEnableChanged)
		{
			EvictAllGeometries(RHICmdList);

			PendingStreamingRequests.Empty();

			SET_DWORD_STAT(STAT_RayTracingPendingStreamingRequests, 0);
		}

		checkf(TotalResidentSize == 0,
			TEXT("TotalResidentSize should be 0 when ray tracing is disabled but is currently %lld.\n")
			TEXT("There's likely some issue tracking resident geometries or not all geometries have been evicted."),
			TotalResidentSize
		);

		check(PendingStreamingRequests.IsEmpty());

		SET_MEMORY_STAT(STAT_RayTracingGeometryReferencedMemory, 0);
		SET_MEMORY_STAT(STAT_RayTracingGeometryRequestedMemory, 0);
		CSV_CUSTOM_STAT(RayTracingGeometry, ReferencedSizeMB, 0.0f, ECsvCustomStatOp::Set);
		CSV_CUSTOM_STAT(RayTracingGeometry, RequestedSizeMB, 0.0f, ECsvCustomStatOp::Set);
	}
	else if (bUsingReferenceBasedResidency)
	{
		check(bIsRayTracingEnabled);

		if (bRenderedFrame)
		{
#if !UE_BUILD_SHIPPING
			if (bDumpUnreferencedAlwaysResidentGeometriesNextFrame)
			{
				bDumpUnreferencedAlwaysResidentGeometriesNextFrame = false;

				DumpUnreferencedAlwaysResidentRayTracingGeometries();
			}
#endif // !UE_BUILD_SHIPPING

			// TODO: swap instead of copying?
			ReferenceBasedResidencyState->ReferencedGeometries = ReferencedGeometries;
			ReferenceBasedResidencyState->ReferencedGeometryGroups = ReferencedGeometryGroups;
			ReferenceBasedResidencyState->ReferencedGeometryGroupsForDynamicUpdate = ReferencedGeometryGroupsForDynamicUpdate;

			ReferenceBasedResidencyTask = UE::Tasks::Launch(UE_SOURCE_LOCATION,
				[ReferenceBasedResidencyState = ReferenceBasedResidencyState.Get()]()
				{
					FTaskTagScope TaskTagScope(ETaskTag::EParallelRenderingThread);
					ReferenceBasedResidencyState->Update();
				});
		}
		else
		{
			ensureMsgf(ReferencedGeometries.IsEmpty() && ReferencedGeometryGroups.IsEmpty() && ReferencedGeometryGroupsForDynamicUpdate.IsEmpty(),
				TEXT("Unexpected entries in ReferencedGeometryHandles/ReferencedGeometryGroups/ReferencedGeometryGroupsForDynamicUpdate. ")
				TEXT("Missing a call to PreRender() or didn't clear the arrays in the last frame?"));
		}
	}
	else
	{
		check(bIsRayTracingEnabled);

		if (bHasRayTracingEnableChanged)
		{
			MakeResidentAllGeometries(RHICmdList);
		}

		SET_MEMORY_STAT(STAT_RayTracingGeometryReferencedMemory, 0);
		SET_MEMORY_STAT(STAT_RayTracingGeometryRequestedMemory, TotalResidentSize);
		CSV_CUSTOM_STAT(RayTracingGeometry, ReferencedSizeMB, 0, ECsvCustomStatOp::Set);
		CSV_CUSTOM_STAT(RayTracingGeometry, RequestedSizeMB, TotalResidentSize / 1024.0f / 1024.0f, ECsvCustomStatOp::Set);
	}

	{
		TSet<RayTracing::FGeometryHandle> CurrentPendingStreamingRequests;
		Swap(CurrentPendingStreamingRequests, PendingStreamingRequests);
		PendingStreamingRequests.Reserve(CurrentPendingStreamingRequests.Num());

		for (RayTracing::FGeometryHandle GeometryHandle : CurrentPendingStreamingRequests)
		{
			if (!RequestRayTracingGeometryStreamIn(RHICmdList, GeometryHandle))
			{
				PendingStreamingRequests.Add(GeometryHandle);
			}
		}
	}

	SET_DWORD_STAT(STAT_RayTracingPendingStreamingRequests, PendingStreamingRequests.Num());

	ProcessCompletedStreamingRequests(RHICmdList);

	ReferencedGeometries.Reset();
	ReferencedGeometryGroups.Reset();
	ReferencedGeometryGroupsForDynamicUpdate.Reset();

	bHasRayTracingEnableChanged = false;
	bRenderedFrame = false;

	SET_MEMORY_STAT(STAT_RayTracingGeometryResidentMemory, TotalResidentSize);
	SET_MEMORY_STAT(STAT_RayTracingGeometryAlwaysResidentMemory, TotalAlwaysResidentSize);

	CSV_CUSTOM_STAT(RayTracingGeometry, TotalResidentSizeMB, TotalResidentSize / 1024.0f / 1024.0f, ECsvCustomStatOp::Set);
	CSV_CUSTOM_STAT(RayTracingGeometry, TotalAlwaysResidentSizeMB, TotalAlwaysResidentSize / 1024.0f / 1024.0f, ECsvCustomStatOp::Set);
}

void FRayTracingGeometryManager::UpdateReferenceBasedResidencyState()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FRayTracingGeometryManager::UpdateReferenceBasedResidencyState);

	ReferenceBasedResidencyState->FrameCounter = GFrameCounterRenderThread;
	ReferenceBasedResidencyState->TotalResidentSize = TotalResidentSize;
	ReferenceBasedResidencyState->TotalAlwaysResidentSize = TotalAlwaysResidentSize;
	ReferenceBasedResidencyState->TotalStreamingSize = TotalStreamingSize;

	// Remove metadata for geometries/groups released during the previous frame
	for (RayTracing::FGeometryHandle Handle : ReleasedGeometries)
	{
		const uint32 GeometryIndex = GetHandleIndex(Handle);

		if (ReferenceBasedResidencyState->GeometryMetadatas.IsValidIndex(GeometryIndex))
		{
			ReferenceBasedResidencyState->GeometryMetadatas.RemoveAt(GeometryIndex);
		}

		ReferenceBasedResidencyState->EvictableGeometries.Remove(Handle);
	}
	ReleasedGeometries.Empty();

	for (RayTracing::FGeometryGroupHandle Handle : ReleasedGroups)
	{
		const uint32 GroupIndex = GetHandleIndex(Handle);

		if (ReferenceBasedResidencyState->GroupMetadatas.IsValidIndex(GroupIndex))
		{
			ReferenceBasedResidencyState->GroupMetadatas.RemoveAt(GroupIndex);
		}
	}
	ReleasedGroups.Empty();

	// Add/update metadata for geometries/groups added/modified during the previous frame.
	for (RayTracing::FGeometryHandle Handle : ModifiedGeometries)
	{
		const FRegisteredGeometry& RegisteredGeometry = GetRegisteredGeometry(Handle);

		const uint32 GeometryIndex = GetHandleIndex(Handle);

		if (!ReferenceBasedResidencyState->GeometryMetadatas.IsValidIndex(GeometryIndex))
		{
			ReferenceBasedResidencyState->GeometryMetadatas.Insert(GeometryIndex, {});
		}

		FReferenceBasedResidencyState::FGeometryMetadata& Metadata = ReferenceBasedResidencyState->GeometryMetadatas[GeometryIndex];
		Metadata.Size = RegisteredGeometry.Size;
		Metadata.GroupHandle = RegisteredGeometry.Geometry->GroupHandle;
		Metadata.bEvicted = RegisteredGeometry.bEvicted;
		Metadata.bBVHStreamedOut = RegisteredGeometry.Geometry->GetInitializer().Type == ERayTracingGeometryInitializerType::StreamingDestination;
		Metadata.bBuffersStreamedOut = RegisteredGeometry.StreamableBuffersSize > 0 && !RegisteredGeometry.StreamableAsset->AreBuffersStreamedIn();
		Metadata.bAlwaysResident = RegisteredGeometry.bAlwaysResident;

		if (RegisteredGeometry.Geometry->IsValid() && !RegisteredGeometry.bEvicted && !RegisteredGeometry.bAlwaysResident)
		{
			ReferenceBasedResidencyState->EvictableGeometries.Add(Handle);
		}
		else
		{
			ReferenceBasedResidencyState->EvictableGeometries.Remove(Handle);
		}
	}
	ModifiedGeometries.Empty();

	for (RayTracing::FGeometryGroupHandle Handle : ModifiedGroups)
	{
		const FRayTracingGeometryGroup& RegisteredGroup = GetRegisteredGroup(Handle);

		const uint32 GroupIndex = GetHandleIndex(Handle);

		if (!ReferenceBasedResidencyState->GroupMetadatas.IsValidIndex(GroupIndex))
		{
			ReferenceBasedResidencyState->GroupMetadatas.Insert(GroupIndex, {});
		}

		FReferenceBasedResidencyState::FGeometryGroupMetadata& Metadata = ReferenceBasedResidencyState->GroupMetadatas[GroupIndex];
		Metadata.GeometryHandles = RegisteredGroup.GeometryHandles;
		Metadata.AdditionalGeometryHandles = RegisteredGroup.AdditionalGeometryHandles;
		Metadata.CurrentFirstLODIdx = RegisteredGroup.CurrentFirstLODIdx;

	}
	ModifiedGroups.Empty();
}

void FRayTracingGeometryManager::ProcessReferenceBasedResidency(FRHICommandList& RHICmdList)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FRayTracingGeometryManager::ProcessReferenceBasedResidency);

	const int64 ReferencedSize = ReferenceBasedResidencyState->ReferencedSize;
	const int64 RequestedSize = ReferenceBasedResidencyState->RequestedSize;
	const int64 RequestedButEvictedSize = ReferenceBasedResidencyState->RequestedButEvictedSize;
	TArray<RayTracing::FGeometryHandle>& GeometriesToEvict = ReferenceBasedResidencyState->GeometriesToEvict;
	TArray<RayTracing::FGeometryHandle>& GeometriesToMakeResidentAndStreamIn = ReferenceBasedResidencyState->GeometriesToMakeResidentAndStreamIn;

	const int64 ResidentGeometryMemoryPoolSize = FUnitConversion::Convert(GRayTracingResidentGeometryMemoryPoolSizeInMB, EUnit::Megabytes, EUnit::Bytes);

	// Evict geometries until we are in budget
	{
		int32 Index = 0;
		while (TotalResidentSize + TotalStreamingSize + RequestedButEvictedSize > ResidentGeometryMemoryPoolSize && Index < GeometriesToEvict.Num())
		{
			RayTracing::FGeometryHandle GeometryHandle = GeometriesToEvict[Index];

			if (IsHandleValid(GeometryHandle))
			{
				FRegisteredGeometry& RegisteredGeometry = GetRegisteredGeometry(GeometryHandle);

				// Can't check RegisteredGeometry.Geometry->IsValid() here since geometry might have been streamed out
				// between UpdateReferenceBasedResidencyState() which ran on the previous frame and now in a different code path.
				check(!RegisteredGeometry.Geometry->IsEvicted());

				EvictGeometry(RHICmdList, RegisteredGeometry);
			}

			++Index;
		}
	}

	// Make referenced geometries resident until we go over budget
	{
		int32 Index = 0;
		while (TotalResidentSize + TotalStreamingSize < ResidentGeometryMemoryPoolSize && Index < GeometriesToMakeResidentAndStreamIn.Num())
		{
			RayTracing::FGeometryHandle GeometryHandle = GeometriesToMakeResidentAndStreamIn[Index];

			if (IsHandleValid(GeometryHandle))
			{
				FRegisteredGeometry& RegisteredGeometry = GetRegisteredGeometry(GeometryHandle);

				if (RegisteredGeometry.Geometry->IsEvicted())
				{
					MakeGeometryResident(RHICmdList, RegisteredGeometry);
				}

				RequestRayTracingGeometryStreamIn(RHICmdList, GeometryHandle);
			}

			++Index;
		}
	}

	GeometriesToEvict.Empty();
	GeometriesToMakeResidentAndStreamIn.Empty();

	SET_MEMORY_STAT(STAT_RayTracingGeometryReferencedMemory, ReferencedSize);
	SET_MEMORY_STAT(STAT_RayTracingGeometryRequestedMemory, RequestedSize);
	CSV_CUSTOM_STAT(RayTracingGeometry, ReferencedSizeMB, ReferencedSize / 1024.0f / 1024.0f, ECsvCustomStatOp::Set);
	CSV_CUSTOM_STAT(RayTracingGeometry, RequestedSizeMB, RequestedSize / 1024.0f / 1024.0f, ECsvCustomStatOp::Set);

#if !UE_BUILD_SHIPPING
	if (GRayTracingShowOnScreenWarnings)
	{
		uint64 MsgId = Murmur64({ (uint64)this, 0x51bc352fe2ed753a }); // To prevent collisions with other on-screen messages.

		const int ThresholdPercentage = FMath::Clamp(CVarRayTracingAlwaysResidentWarningPercentage.GetValueOnRenderThread(), 0, 100);
		if (TotalAlwaysResidentSize > ResidentGeometryMemoryPoolSize * ThresholdPercentage / 100.0f)
		{
			GEngine->AddOnScreenDebugMessage(MsgId++, 1.f, FColor::Red,
				*FString::Printf(TEXT("RAY TRACING GEOMETRY - ALWAYS RESIDENT MEMORY EXCEEDS %d%% OF THE BUDGET (%s / %s)"),
					ThresholdPercentage,
					*FText::AsMemory(TotalAlwaysResidentSize).ToString(),
					*FText::AsMemory(ResidentGeometryMemoryPoolSize).ToString()));
		}

		if (RequestedSize > ResidentGeometryMemoryPoolSize)
		{
			GEngine->AddOnScreenDebugMessage(MsgId++, 1.f, FColor::Yellow,
				*FString::Printf(TEXT("RAY TRACING GEOMETRY - REQUESTED MEMORY OVER BUDGET %s / %s"),
					*FText::AsMemory(RequestedSize).ToString(),
					*FText::AsMemory(ResidentGeometryMemoryPoolSize).ToString()));
		}
	}
#endif
}

void FRayTracingGeometryManager::MakeResidentAllGeometries(FRHICommandList& RHICmdList)
{
	for (FRegisteredGeometry& RegisteredGeometry : RegisteredGeometries)
	{
		if (RegisteredGeometry.Geometry->IsEvicted())
		{
			MakeGeometryResident(RHICmdList, RegisteredGeometry);
		}

		if (!RequestRayTracingGeometryStreamIn(RHICmdList, RegisteredGeometry.Geometry->RayTracingGeometryHandle))
		{
			PendingStreamingRequests.Add(RegisteredGeometry.Geometry->RayTracingGeometryHandle);

			INC_DWORD_STAT(STAT_RayTracingPendingStreamingRequests);
		}
	}
}

void FRayTracingGeometryManager::EvictAllGeometries(FRHICommandList& RHICmdList)
{
	for (FRegisteredGeometry& RegisteredGeometry : RegisteredGeometries)
	{
		if (RegisteredGeometry.Geometry->GetRHI() != nullptr)
		{
			EvictGeometry(RHICmdList, RegisteredGeometry);
		}
	}
}

void FRayTracingGeometryManager::RefreshAlwaysResidentRayTracingGeometries(FRHICommandList& RHICmdList)
{
	const bool bUsingReferenceBasedResidency = IsRayTracingUsingReferenceBasedResidency();

	if (bRefreshAlwaysResidentRayTracingGeometries)
	{
		bRefreshAlwaysResidentRayTracingGeometries = false;

		AlwaysResidentGeometries.Empty();
		TotalAlwaysResidentSize = 0;

		for (FRegisteredGeometry& RegisteredGeometry : RegisteredGeometries)
		{
			const RayTracing::FGeometryGroupHandle GroupHandle = RegisteredGeometry.Geometry->GroupHandle;

			if (GroupHandle.IsNull())
			{
				checkf(!RegisteredGeometry.bAlwaysResident, TEXT("Ray tracing geometry unexpectedly marked as always-resident even though it is not assigned to a group."));
				continue;
			}

			const FRayTracingGeometryGroup& Group = GetRegisteredGroup(GroupHandle);

			RegisteredGeometry.bAlwaysResident = IsAlwaysResidentGeometry(RegisteredGeometry.Geometry, Group);

			if (RegisteredGeometry.bAlwaysResident)
			{
				AlwaysResidentGeometries.Add(RegisteredGeometry.Geometry->RayTracingGeometryHandle);
				TotalAlwaysResidentSize += RegisteredGeometry.Size;

				if (RegisteredGeometry.Geometry->IsEvicted())
				{
					MakeGeometryResident(RHICmdList, RegisteredGeometry);
				}

				if (!RequestRayTracingGeometryStreamIn(RHICmdList, RegisteredGeometry.Geometry->RayTracingGeometryHandle))
				{
					PendingStreamingRequests.Add(RegisteredGeometry.Geometry->RayTracingGeometryHandle);

					INC_DWORD_STAT(STAT_RayTracingPendingStreamingRequests);
				}
			}
			else if (bUsingReferenceBasedResidency && RegisteredGeometry.Geometry->GetRHI() != nullptr)
			{
				EvictGeometry(RHICmdList, RegisteredGeometry);
			}

			ModifiedGeometries.Add(RegisteredGeometry.Geometry->RayTracingGeometryHandle);
		}
	}
	else if (bUsingReferenceBasedResidency && bHasRayTracingEnableChanged)
	{
		// make always-resident geometries actually resident

		for (RayTracing::FGeometryHandle GeometryHandle : AlwaysResidentGeometries)
		{
			FRegisteredGeometry& RegisteredGeometry = GetRegisteredGeometry(GeometryHandle);

			if (RegisteredGeometry.Geometry->IsEvicted())
			{
				MakeGeometryResident(RHICmdList, RegisteredGeometry);
			}
			if (!RequestRayTracingGeometryStreamIn(RHICmdList, GeometryHandle))
			{
				PendingStreamingRequests.Add(GeometryHandle);

				INC_DWORD_STAT(STAT_RayTracingPendingStreamingRequests);
			}
		}
	}
}

#if DO_CHECK
void FRayTracingGeometryManager::CheckIntegrity() const
{
	const bool bIsRayTracingEnabled = IsRayTracingEnabled();
	const bool bUsingReferenceBasedResidency = IsRayTracingUsingReferenceBasedResidency();

	if (GRayTracingTestCheckIntegrity)
	{
		uint32 NumAlwaysResidentGeometries = 0;

		for (const FRegisteredGeometry& RegisteredGeometry : RegisteredGeometries)
		{
			const RayTracing::FGeometryHandle GeometryHandle = RegisteredGeometry.Geometry->RayTracingGeometryHandle;
			const RayTracing::FGeometryGroupHandle GroupHandle = RegisteredGeometry.Geometry->GroupHandle;

			bool bAlwaysResident = false;

			if (!GroupHandle.IsNull())
			{
				const FRayTracingGeometryGroup& Group = GetRegisteredGroup(GroupHandle);
				bAlwaysResident = IsAlwaysResidentGeometry(RegisteredGeometry.Geometry, Group);
			}
			else
			{
				// geometries not assigned to a group (eg: dynamic geometry) are always evictable
			}

			checkf(RegisteredGeometry.bEvicted == RegisteredGeometry.Geometry->IsEvicted(), TEXT("Cached bEvicted flag in FRegisteredGeometry is stale"));
			checkf(bAlwaysResident == RegisteredGeometry.bAlwaysResident, TEXT("Cached bAlwaysResident flag in FRegisteredGeometry is stale"));
			checkf(bAlwaysResident == AlwaysResidentGeometries.Contains(GeometryHandle), TEXT("Geometry with bAlwaysResident flag set should be in the AlwaysResidentGeometries set."));

			if (bAlwaysResident)
			{
				++NumAlwaysResidentGeometries;
			}
		}

		check(NumAlwaysResidentGeometries == AlwaysResidentGeometries.Num());

		uint32 NumEvictableGeometries = 0;

		for (auto It = RegisteredGeometries.CreateConstIterator(); It; ++It)
		{
			const FRegisteredGeometry& RegisteredGeometry = *It;
			const uint32 GeometryIndex = It.GetIndex();

			const RayTracing::FGeometryHandle GeometryHandle = RegisteredGeometry.Geometry->RayTracingGeometryHandle;
			const RayTracing::FGeometryGroupHandle GroupHandle = RegisteredGeometry.Geometry->GroupHandle;

			const bool bBVHStreamedOut = RegisteredGeometry.Geometry->GetInitializer().Type == ERayTracingGeometryInitializerType::StreamingDestination;
			const bool bBuffersStreamedOut = RegisteredGeometry.StreamableBuffersSize > 0 && !RegisteredGeometry.StreamableAsset->AreBuffersStreamedIn();

			const FReferenceBasedResidencyState::FGeometryMetadata& Metadata = ReferenceBasedResidencyState->GeometryMetadatas[GeometryIndex];
			check(Metadata.Size == RegisteredGeometry.Size);
			check(Metadata.GroupHandle == RegisteredGeometry.Geometry->GroupHandle);
			check(Metadata.bEvicted == RegisteredGeometry.bEvicted);
			check(Metadata.bBVHStreamedOut == bBVHStreamedOut);
			check(Metadata.bBuffersStreamedOut == bBuffersStreamedOut);
			check(Metadata.bAlwaysResident == RegisteredGeometry.bAlwaysResident);

			if (RegisteredGeometry.Geometry->IsValid() && !RegisteredGeometry.bEvicted && !RegisteredGeometry.bAlwaysResident)
			{
				check(ReferenceBasedResidencyState->EvictableGeometries.Contains(GeometryHandle));
				++NumEvictableGeometries;
			}
			else
			{
				check(!ReferenceBasedResidencyState->EvictableGeometries.Contains(GeometryHandle));
			}
		}

		check(NumEvictableGeometries == ReferenceBasedResidencyState->EvictableGeometries.Num());

		for (auto It = RegisteredGroups.CreateConstIterator(); It; ++It)
		{
			const FRayTracingGeometryGroup& RegisteredGroup = *It;
			const uint32 GroupIndex = It.GetIndex();

			FReferenceBasedResidencyState::FGeometryGroupMetadata& Metadata = ReferenceBasedResidencyState->GroupMetadatas[GroupIndex];
			check(Metadata.GeometryHandles == RegisteredGroup.GeometryHandles);
			check(Metadata.CurrentFirstLODIdx == RegisteredGroup.CurrentFirstLODIdx);
		}

		if (!bIsRayTracingEnabled)
		{
			// check that everything is evicted
			for (const FRegisteredGeometry& RegisteredGeometry : RegisteredGeometries)
			{
				checkf(RegisteredGeometry.Geometry->IsEvicted() || RegisteredGeometry.Geometry->GetRHI() == nullptr, TEXT("Ray tracing geometry should be evicted when ray tracing is disabled."));
			}
		}
		else if (!bUsingReferenceBasedResidency)
		{
			// check that all geometries are resident
			for (const FRegisteredGeometry& RegisteredGeometry : RegisteredGeometries)
			{
				checkf(!RegisteredGeometry.Geometry->IsEvicted(), TEXT("Ray tracing geometry should not be evicted when ray tracing is enabled."));
			}
		}
	}
}
#endif // DO_CHECK

void FRayTracingGeometryManager::DoesRayTracingGeometryRequireStreaming(const FRegisteredGeometry& RegisteredGeometry, bool& bOutStreamBVH, bool& bOutStreamBuffers)
{
	bOutStreamBVH = false;
	bOutStreamBuffers = false;

	if (RegisteredGeometry.StreamingRequestIndex != INDEX_NONE)
	{
		// skip if there's already a streaming request in-flight for this geometry
		return;
	}

	bOutStreamBVH = RegisteredGeometry.Geometry->GetInitializer().Type == ERayTracingGeometryInitializerType::StreamingDestination;
	bOutStreamBuffers = bOutStreamBVH || (RegisteredGeometry.StreamableBuffersSize > 0 && !RegisteredGeometry.StreamableAsset->AreBuffersStreamedIn() && ReferencedGeometryGroupsForDynamicUpdate.Contains(RegisteredGeometry.Geometry->GroupHandle));

	if (!bOutStreamBVH && !bOutStreamBuffers)
	{
		// no streaming required
		return;
	}

	if (!RegisteredGeometry.Geometry->GroupHandle.IsNull())
	{
		const FRayTracingGeometryGroup& Group = GetRegisteredGroup(RegisteredGeometry.Geometry->GroupHandle);

		if (!IsAdditionalRayTracingGeometry(RegisteredGeometry.Geometry) && RegisteredGeometry.Geometry->LODIndex < Group.CurrentFirstLODIdx)
		{
			bOutStreamBVH = false;
			bOutStreamBuffers = false;
			return;
		}
	}
}

bool FRayTracingGeometryManager::RequestRayTracingGeometryStreamIn(FRHICommandList& RHICmdList, RayTracing::FGeometryHandle GeometryHandle)
{
	FRegisteredGeometry& RegisteredGeometry = GetRegisteredGeometry(GeometryHandle);
	FRayTracingGeometry* Geometry = RegisteredGeometry.Geometry;

	bool bStreamBVH;
	bool bStreamBuffers;
	DoesRayTracingGeometryRequireStreaming(RegisteredGeometry, bStreamBVH, bStreamBuffers);

	if (!bStreamBuffers && !bStreamBVH)
	{
		// no streaming required
		return true;
	}

	// TODO: Support DDC streaming

	if (RegisteredGeometry.StreamableBuffersSize == 0 && RegisteredGeometry.StreamableBVHSize == 0)
	{
		// no offline data -> build from VB/IB at runtime

		{
			FRHIResourceReplaceBatcher Batcher(RHICmdList, 1);
			FRayTracingGeometryInitializer IntermediateInitializer = Geometry->GetInitializer();
			IntermediateInitializer.Type = ERayTracingGeometryInitializerType::StreamingSource;
			IntermediateInitializer.OfflineData = nullptr;

			FRayTracingGeometryRHIRef IntermediateRayTracingGeometry = RHICmdList.CreateRayTracingGeometry(IntermediateInitializer);

			Geometry->SetRequiresBuild(IntermediateInitializer.OfflineData == nullptr || IntermediateRayTracingGeometry->IsCompressed());

			Geometry->InitRHIForStreaming(IntermediateRayTracingGeometry, Batcher);

			// When Batcher goes out of scope it will add commands to copy the BLAS buffers on RHI thread.
			// We need to do it before we build the current geometry (also on RHI thread).
		}

		Geometry->RequestBuildIfNeeded(RHICmdList, ERTAccelerationStructureBuildPriority::Normal);
	}
	else
	{
		if (NumStreamingRequests >= GRayTracingStreamingMaxPendingRequests)
		{
			return false;
		}

		checkf(RegisteredGeometry.StreamingRequestIndex == INDEX_NONE, TEXT("Ray Tracing Geometry already has a streaming request in-flight"));
		RegisteredGeometry.StreamingRequestIndex = NextStreamingRequestIndex;

		FStreamingRequest& StreamingRequest = StreamingRequests[NextStreamingRequestIndex];
		checkf(!StreamingRequest.IsValid(), TEXT("Unused streaming request are expected to be in invalid state."));
		NextStreamingRequestIndex = (NextStreamingRequestIndex + 1) % GRayTracingStreamingMaxPendingRequests;
		++NumStreamingRequests;

		INC_DWORD_STAT(STAT_RayTracingInflightStreamingRequests);

		uint32 StreamableDataSize = 0;

		if (bStreamBuffers)
		{
			StreamableDataSize += RegisteredGeometry.StreamableBuffersSize;
		}

		if (bStreamBVH)
		{
			check(!RegisteredGeometry.StreamableAsset->IsBVHStreamedIn());
			StreamableDataSize += RegisteredGeometry.StreamableBVHSize;
		}

		StreamingRequest.GeometryHandle = GeometryHandle;
		StreamingRequest.GeometrySize = RegisteredGeometry.Size;
		StreamingRequest.bBuffersOnly = !bStreamBVH;
		StreamingRequest.RequestBuffer = FIoBuffer(StreamableDataSize); // TODO: Use FIoBuffer::Wrap with preallocated memory

		EAsyncIOPriorityAndFlags Priority = RegisteredGeometry.bAlwaysResident ? AIOP_Normal : AIOP_Low;

		RegisteredGeometry.StreamableAsset->IssueRequest(StreamingRequest.Request, StreamingRequest.RequestBuffer, Priority, StreamingRequest.bBuffersOnly);

		TotalStreamingSize += StreamingRequest.GeometrySize;
	}

	return true;
}

void FRayTracingGeometryManager::ProcessCompletedStreamingRequests(FRHICommandList& RHICmdList)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FRayTracingGeometryManager::ProcessCompletedStreamingRequests);

	const bool bOnDemandGeometryBuffersStreaming = IsRayTracingUsingReferenceBasedResidency() && CVarRayTracingOnDemandGeometryBuffersStreaming.GetValueOnRenderThread();

	const int32 StartPendingRequestIndex = (NextStreamingRequestIndex + GRayTracingStreamingMaxPendingRequests - NumStreamingRequests) % GRayTracingStreamingMaxPendingRequests;

	int32 NumCompletedRequests = 0;

	for (int32 Index = 0; Index < NumStreamingRequests; ++Index)
	{
		const int32 PendingRequestIndex = (StartPendingRequestIndex + Index) % GRayTracingStreamingMaxPendingRequests;
		FStreamingRequest& PendingRequest = StreamingRequests[PendingRequestIndex];

		checkf(PendingRequest.IsValid(), TEXT("Pending streaming request should be valid."));

		if (PendingRequest.Request.IsCompleted())
		{
			++NumCompletedRequests;

			TotalStreamingSize -= PendingRequest.GeometrySize;
			check(TotalStreamingSize >= 0);

			if (PendingRequest.bCancelled)
			{
				PendingRequest.Reset();
				continue;
			}

			FRegisteredGeometry& RegisteredGeometry = GetRegisteredGeometry(PendingRequest.GeometryHandle);

			RegisteredGeometry.StreamingRequestIndex = INDEX_NONE;

			const FRayTracingGeometryGroup& Group = GetRegisteredGroup(RegisteredGeometry.Geometry->GroupHandle);

			if (RegisteredGeometry.Geometry->IsEvicted() || (!IsAdditionalRayTracingGeometry(RegisteredGeometry.Geometry) && RegisteredGeometry.Geometry->LODIndex < Group.CurrentFirstLODIdx))
			{
				// do nothing since geometry was evicted while streaming request was being processed
			}
			else if (!PendingRequest.Request.IsOk())
			{
				UE_LOGF(LogRayTracingGeometryManager, Warning, "Ray Tracing Geometry IO Request failed (%ls)", *RegisteredGeometry.Geometry->GetInitializer().DebugName.ToString());

				// Manager will retry again if still necessary on the next frame
			}
			else
			{
				RegisteredGeometry.StreamableAsset->InitWithStreamedData(RHICmdList, PendingRequest.RequestBuffer.GetView(), PendingRequest.bBuffersOnly);

				// if VB/IB are not being used for dynamic BLAS updates (eg: WPO)
				// and the RHI doesn't need them either (hit shaders not supported / inline SBT not required)
				// then we can stream-out the buffers after BLAS is built
				if (!GRHIGlobals.RayTracing.SupportsShaders
					&& !GRHIGlobals.RayTracing.RequiresInlineRayTracingSBT
					&& bOnDemandGeometryBuffersStreaming
					&& !ReferencedGeometryGroupsForDynamicUpdate.Contains(RegisteredGeometry.Geometry->GroupHandle))
				{
					if (RegisteredGeometry.Geometry->HasPendingBuildRequest())
					{
						// need to delay releasing buffers until build is dispatched
						FBuildRequest& BuildRequest = GetBuildRequest(RegisteredGeometry.Geometry->RayTracingBuildRequestIndex);
						BuildRequest.bReleaseBuffersAfterBuild = true;
					}
					else
					{
						FRHIResourceReplaceBatcher Batcher(RHICmdList, 1);
						RegisteredGeometry.StreamableAsset->ReleaseBuffersForStreaming(Batcher);
					}
				}
			}

			PendingRequest.Reset();
		}
		else
		{
			// TODO: Could other requests already be completed?
			break;
		}
	}

	NumStreamingRequests -= NumCompletedRequests;

	SET_DWORD_STAT(STAT_RayTracingInflightStreamingRequests, NumStreamingRequests);
}

void FRayTracingGeometryManager::CancelStreamingRequest(FRegisteredGeometry& RegisteredGeometry)
{
	if (RegisteredGeometry.StreamingRequestIndex != INDEX_NONE)
	{
		FStreamingRequest& StreamingRequest = StreamingRequests[RegisteredGeometry.StreamingRequestIndex];
		checkf(StreamingRequest.GeometryHandle == RegisteredGeometry.Geometry->RayTracingGeometryHandle,
			TEXT("Ray tracing geometry streaming request owner mismatch (expected (%u, %u), got (%u, %u))."),
			GetHandleIndex(RegisteredGeometry.Geometry->RayTracingGeometryHandle), 
			GetHandleVersion(RegisteredGeometry.Geometry->RayTracingGeometryHandle), 
			GetHandleIndex(StreamingRequest.GeometryHandle),
			GetHandleVersion(StreamingRequest.GeometryHandle));

		StreamingRequest.Cancel();

		RegisteredGeometry.StreamingRequestIndex = INDEX_NONE;
	}
}

void FRayTracingGeometryManager::StreamOutGeometry(FRHIResourceReplaceBatcher& Batcher, FRegisteredGeometry& RegisteredGeometry)
{
	if (EnumHasAllFlags(RegisteredGeometry.Geometry->GetGeometryState(), FRayTracingGeometry::EGeometryStateFlags::StreamedIn))
	{
		if (RegisteredGeometry.StreamableAsset != nullptr)
		{
			RegisteredGeometry.StreamableAsset->ReleaseForStreaming(Batcher);
		}
		else
		{
			RegisteredGeometry.Geometry->ReleaseRHIForStreaming(Batcher);
		}
	}
}

void FRayTracingGeometryManager::MakeGeometryResident(FRHICommandList& RHICmdList, FRegisteredGeometry& RegisteredGeometry)
{
	RegisteredGeometry.Geometry->MakeResident(RHICmdList);
	RegisteredGeometry.bEvicted = false;
}

void FRayTracingGeometryManager::EvictGeometry(FRHICommandListBase& RHICmdList, FRegisteredGeometry& RegisteredGeometry)
{
	// Cancel associated streaming request if currently in-flight
	CancelStreamingRequest(RegisteredGeometry);

	// Both FRayTracingGeometry::ReleaseRHIForStreaming(...) and FRayTracingGeometry::Evict()
	// call FRayTracingGeometryManager:::RefreshRegisteredGeometry(...) which is unecessary
	// however there's no straightforward way to avoid that
	// TODO: investigate possible improvements

	FRHIResourceReplaceBatcher Batcher(RHICmdList, 1);
	StreamOutGeometry(Batcher, RegisteredGeometry);

	RegisteredGeometry.Geometry->Evict();
	check(RegisteredGeometry.bEvicted);
}

void FRayTracingGeometryManager::BoostPriority(RayTracing::FBuildRequestHandle InRequestHandle, float InBoostValue)
{
	FScopeLock ScopeLock(&RequestCS);

	FBuildRequest& BuildRequest = GetBuildRequest(InRequestHandle);
	BuildRequest.BuildPriority += InBoostValue;
}

void FRayTracingGeometryManager::ForceBuildIfPending(FRHIComputeCommandList& InCmdList, const TArrayView<const FRayTracingGeometry*> InGeometries)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FRayTracingGeometryManager::ForceBuildIfPending);

	FScopeLock ScopeLock(&RequestCS);

	TArray<FRayTracingGeometry*> ReleaseBuffers;

	BuildParams.Empty(FMath::Max(BuildParams.Max(), InGeometries.Num()));
	for (const FRayTracingGeometry* Geometry : InGeometries)
	{
		if (Geometry->HasPendingBuildRequest())
		{
			const uint32 RequestIndex = GetHandleIndex(Geometry->RayTracingBuildRequestIndex);
			SetupBuildParams(GetBuildRequest(Geometry->RayTracingBuildRequestIndex), BuildParams, ReleaseBuffers);
			GeometryBuildRequests.RemoveAt(RequestIndex);
		}
	}

	if (BuildParams.Num())
	{
		InCmdList.BuildAccelerationStructures(BuildParams);
	}

	BuildParams.Reset();

	for (FRayTracingGeometry* Geometry : ReleaseBuffers)
	{
		FRegisteredGeometry& RegisteredGeometry = GetRegisteredGeometry(Geometry->RayTracingGeometryHandle);

		FRHIResourceReplaceBatcher Batcher(InCmdList, 1);
		RegisteredGeometry.StreamableAsset->ReleaseBuffersForStreaming(Batcher);
	}
}

void FRayTracingGeometryManager::ProcessBuildRequests(FRHIComputeCommandList& InCmdList, bool bInBuildAll)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FRayTracingGeometryManager::ProcessBuildRequests);

	FScopeLock ScopeLock(&RequestCS);

	if (GeometryBuildRequests.Num() == 0)
	{
		return;
	}

	checkf(BuildParams.IsEmpty(), TEXT("Unexpected entries in BuildParams. The array should've been reset at the end of the previous call."));
	checkf(SortedRequests.IsEmpty(), TEXT("Unexpected entries in SortedRequests. The array should've been reset at the end of the previous call."));

	TArray<FRayTracingGeometry*> ReleaseBuffers;

	BuildParams.Empty(FMath::Max(BuildParams.Max(), GeometryBuildRequests.Num()));

	if (GRayTracingMaxBuiltPrimitivesPerFrame <= 0)
	{
		// no limit -> no need to sort

		SortedRequests.Empty(); // free potentially allocated memory

		for (FBuildRequest& Request : GeometryBuildRequests)
		{
			SetupBuildParams(Request, BuildParams, ReleaseBuffers);
		}

		// after setting up build params can clear the whole array
		GeometryBuildRequests.Reset();
	}
	else
	{
		SortedRequests.Empty(FMath::Max(SortedRequests.Max(), GeometryBuildRequests.Num()));

		{
			TRACE_CPUPROFILER_EVENT_SCOPE(SortRequests);

			// Is there a fast way to extract all entries from sparse array?
			for (const FBuildRequest& Request : GeometryBuildRequests)
			{
				SortedRequests.Add(Request);
			}

			SortedRequests.Sort([](const FBuildRequest& InLHS, const FBuildRequest& InRHS)
				{
					return InLHS.BuildPriority > InRHS.BuildPriority;
				});
		}

		// process n requests each 'frame'
		uint64 PrimitivesBuild = 0;
		bool bAddBuildRequest = true;
		for (FBuildRequest& Request : SortedRequests)
		{
			if (bAddBuildRequest || Request.BuildPriority >= 1.0f) // always build immediate requests
			{
				const uint32 RequestIndex = GetHandleIndex(Request.RequestHandle);
				SetupBuildParams(Request, BuildParams, ReleaseBuffers);
				GeometryBuildRequests.RemoveAt(RequestIndex);

				// Requested enough?
				PrimitivesBuild += Request.Owner->GetInitializer().TotalPrimitiveCount;
				if (!bInBuildAll && (PrimitivesBuild > GRayTracingMaxBuiltPrimitivesPerFrame))
				{
					bAddBuildRequest = false;
				}
			}
			else
			{
				// Increment priority to make sure requests don't starve
				Request.BuildPriority += GRayTracingPendingBuildPriorityBoostPerFrame;
			}
		}

		SortedRequests.Reset();
	}

	// kick actual build request to RHI command list
	if (BuildParams.Num())
	{
		InCmdList.BuildAccelerationStructures(BuildParams);
	}

	BuildParams.Reset();

	for (FRayTracingGeometry* Geometry : ReleaseBuffers)
	{
		FRegisteredGeometry& RegisteredGeometry = GetRegisteredGeometry(Geometry->RayTracingGeometryHandle);

		FRHIResourceReplaceBatcher Batcher(InCmdList, 1);
		RegisteredGeometry.StreamableAsset->ReleaseBuffersForStreaming(Batcher);
	}
}

void FRayTracingGeometryManager::SetupBuildParams(const FBuildRequest& InBuildRequest, TArray<FRayTracingGeometryBuildParams>& InBuildParams, TArray<FRayTracingGeometry*>& InReleaseBuffers)
{
	FRHIRayTracingGeometry* RayTracingGeometryRHI = InBuildRequest.Owner->GetRHI();
	const FRayTracingGeometryInitializer& GeometryInitializer = InBuildRequest.Owner->GetInitializer();

	check(!InBuildRequest.RequestHandle.IsNull() && InBuildRequest.RequestHandle == InBuildRequest.Owner->RayTracingBuildRequestIndex);
	checkf(RayTracingGeometryRHI != nullptr, TEXT("Build request for FRayTracingGeometry without valid RHI. Was the FRayTracingGeometry evicted or released without calling RemoveBuildRequest()?"));

	InBuildRequest.Owner->RayTracingBuildRequestIndex = {};

	DEC_DWORD_STAT(STAT_RayTracingPendingBuilds);
	DEC_DWORD_STAT_BY(STAT_RayTracingPendingBuildPrimitives, GeometryInitializer.TotalPrimitiveCount);

	const uint64 MaxScratchBufferSize = 2147483647u;
	const uint64 RequiredScratchBufferSize = InBuildRequest.BuildMode == EAccelerationStructureBuildMode::Build
		? RayTracingGeometryRHI->GetSizeInfo().BuildScratchSize
		: RayTracingGeometryRHI->GetSizeInfo().UpdateScratchSize;

	if (RequiredScratchBufferSize > MaxScratchBufferSize)
	{
		UE_LOGF(LogRayTracingGeometryManager, Warning, "Ray Tracing Geometry (%ls) with %d primitives requires too large scratch buffer (%llu) - skipping the build.",
			*GeometryInitializer.DebugName.ToString(),
			GeometryInitializer.TotalPrimitiveCount, RequiredScratchBufferSize);
		return;
	}

	FRayTracingGeometryBuildParams BuildParam;
	BuildParam.Geometry = RayTracingGeometryRHI;
	BuildParam.BuildMode = InBuildRequest.BuildMode;
	InBuildParams.Add(BuildParam);

	if (!InBuildRequest.Owner->GroupHandle.IsNull())
	{
		RequestUpdateCachedRenderState(InBuildRequest.Owner->GroupHandle);
	}

	if (InBuildRequest.bReleaseBuffersAfterBuild)
	{
		InReleaseBuffers.Add(InBuildRequest.Owner);
	}
}

void FRayTracingGeometryManager::RegisterProxyWithCachedRayTracingState(FPrimitiveSceneProxy* Proxy, RayTracing::FGeometryGroupHandle InRayTracingGeometryGroupHandle)
{
	checkf(IsInRenderingThread(), TEXT("Can only access RegisteredGroups on render thread otherwise need a critical section"));
	checkf(IsRayTracingAllowed(), TEXT("Should only register proxies with FRayTracingGeometryManager when ray tracing is allowed"));

	FRayTracingGeometryGroup& Group = GetRegisteredGroup(InRayTracingGeometryGroupHandle);

	TSet<FPrimitiveSceneProxy*>& ProxiesSet = Group.ProxiesWithCachedRayTracingState;
	check(!ProxiesSet.Contains(Proxy));

	ProxiesSet.Add(Proxy);

	++Group.NumReferences;
}

void FRayTracingGeometryManager::UnregisterProxyWithCachedRayTracingState(FPrimitiveSceneProxy* Proxy, RayTracing::FGeometryGroupHandle InRayTracingGeometryGroupHandle)
{
	checkf(IsInRenderingThread(), TEXT("Can only access RegisteredGroups on render thread otherwise need a critical section"));
	checkf(IsRayTracingAllowed(), TEXT("Should only register proxies with FRayTracingGeometryManager when ray tracing is allowed"));

	FRayTracingGeometryGroup& Group = GetRegisteredGroup(InRayTracingGeometryGroupHandle);

	TSet<FPrimitiveSceneProxy*>& ProxiesSet = Group.ProxiesWithCachedRayTracingState;

	verify(ProxiesSet.Remove(Proxy) == 1);

	ReleaseRayTracingGeometryGroupReference(InRayTracingGeometryGroupHandle);
}

void FRayTracingGeometryManager::RequestUpdateCachedRenderState(RayTracing::FGeometryGroupHandle InRayTracingGeometryGroupHandle)
{
	checkf(IsInRenderingThread(), TEXT("Can only access RegisteredGroups on render thread otherwise need a critical section"));
	checkf(IsRayTracingAllowed(), TEXT("Should only register proxies with FRayTracingGeometryManager when ray tracing is allowed"));

	const FRayTracingGeometryGroup& Group = GetRegisteredGroup(InRayTracingGeometryGroupHandle);

	const TSet<FPrimitiveSceneProxy*>& ProxiesSet = Group.ProxiesWithCachedRayTracingState;

	for (FPrimitiveSceneProxy* Proxy : ProxiesSet)
	{
		Proxy->GetScene().UpdateCachedRayTracingState(Proxy);
	}
}

void FRayTracingGeometryManager::AddReferencedGeometry(const FRayTracingGeometry* Geometry)
{
	check(IsInRenderingThread() || IsInParallelRenderingThread());

	if (IsRayTracingUsingReferenceBasedResidency())
	{
		if (!Geometry->RayTracingGeometryHandle.IsNull())
		{
#if DO_CHECK
			// try to use handle here in development builds to detect handles as early as possible
			GetRegisteredGeometry(Geometry->RayTracingGeometryHandle);
#endif

			ReferencedGeometries.Add(Geometry->RayTracingGeometryHandle);
		}
	}
	else
	{
		ensureMsgf(ReferencedGeometries.IsEmpty(), TEXT("Should only track ReferencedGeometries when using reference based residency"));
	}
}

void FRayTracingGeometryManager::AddReferencedGeometries(const TSet<RayTracing::FGeometryHandle>& Geometries)
{
	check(IsInRenderingThread() || IsInParallelRenderingThread());

	if (IsRayTracingUsingReferenceBasedResidency())
	{
#if DO_CHECK
		// try to use handle here in development builds to detect handles as early as possible
		for (RayTracing::FGeometryHandle Handle : Geometries)
		{
			GetRegisteredGeometry(Handle);
		}
#endif

		ReferencedGeometries.Append(Geometries);
	}
	else
	{
		ensureMsgf(Geometries.IsEmpty(), TEXT("Should only track ReferencedGeometries when using reference based residency"));
	}
}

void FRayTracingGeometryManager::AddReferencedGeometryGroups(const TSet<RayTracing::FGeometryGroupHandle>& GeometryGroups)
{
	check(IsInRenderingThread() || IsInParallelRenderingThread());

	if (IsRayTracingUsingReferenceBasedResidency())
	{
#if DO_CHECK
		// try to use handle here in development builds to detect handles as early as possible
		for (RayTracing::FGeometryGroupHandle Handle : GeometryGroups)
		{
			GetRegisteredGroup(Handle);
		}
#endif

		ReferencedGeometryGroups.Append(GeometryGroups);
	}
	else
	{
		ensureMsgf(GeometryGroups.IsEmpty(), TEXT("Should only track ReferencedGeometryGroups when using reference based residency"));
	}
}

void FRayTracingGeometryManager::AddReferencedGeometryGroupsForDynamicUpdate(const TSet<RayTracing::FGeometryGroupHandle>& GeometryGroups)
{
	check(IsInRenderingThread() || IsInParallelRenderingThread());

	if (IsRayTracingUsingReferenceBasedResidency())
	{
#if DO_CHECK
		// try to use handle here in development builds to detect handles as early as possible
		for (RayTracing::FGeometryGroupHandle Handle : GeometryGroups)
		{
			GetRegisteredGroup(Handle);
		}
#endif

		ReferencedGeometryGroupsForDynamicUpdate.Append(GeometryGroups);
	}
	else
	{
		ensureMsgf(GeometryGroups.IsEmpty(), TEXT("Should only track ReferencedGeometryGroupsForDynamic when using reference based residency"));
	}
}

bool FRayTracingGeometryManager::IsGeometryVisible(RayTracing::FGeometryHandle GeometryHandle) const
{
	return VisibleGeometryHandles.Contains(GeometryHandle);
}


void FRayTracingGeometryManager::AddVisibleGeometry(RayTracing::FGeometryHandle GeometryHandle)
{
	VisibleGeometryHandles.Add(GeometryHandle);
}

void FRayTracingGeometryManager::ResetVisibleGeometries()
{
	// Reset the previous frame handles
	VisibleGeometryHandles.Empty(VisibleGeometryHandles.Num());
}

#if DO_CHECK
bool FRayTracingGeometryManager::IsGeometryReferenced(const FRayTracingGeometry* Geometry) const
{
	return ReferencedGeometries.Contains(Geometry->RayTracingGeometryHandle);
}

bool FRayTracingGeometryManager::IsGeometryGroupReferenced(RayTracing::FGeometryGroupHandle GeometryGroup) const
{
	return ReferencedGeometryGroups.Contains(GeometryGroup);
}
#endif // DO_CHECK

#if !UE_BUILD_SHIPPING
bool FRayTracingGeometryManager::DumpRayTracingGeometries(TArrayView<RayTracing::FGeometryHandle> InGeometries, const FString& Filename) const
{
	Algo::Sort(InGeometries, [this](RayTracing::FGeometryHandle& LHSHandle, RayTracing::FGeometryHandle& RHSHandle)
		{
			const FRegisteredGeometry& LHS = GetRegisteredGeometry(LHSHandle);
			const FRegisteredGeometry& RHS = GetRegisteredGeometry(RHSHandle);

			return LHS.Size > RHS.Size;
		});

	TUniquePtr<FArchive> CSVFile = TUniquePtr<FArchive>(IFileManager::Get().CreateFileWriter(*Filename, FILEWRITE_AllowRead));
	if (!CSVFile)
	{
		return false;
	}

	const TCHAR* Header = TEXT("Name,Size (MBs),Prims,Segments,Update\n");
	CSVFile->Serialize(TCHAR_TO_ANSI(Header), FPlatformString::Strlen(Header));

	for (RayTracing::FGeometryHandle GeometryHandle : InGeometries)
	{
		const FRegisteredGeometry& RegisteredGeometry = GetRegisteredGeometry(GeometryHandle);
		const FRayTracingGeometryInitializer Initializer = RegisteredGeometry.Geometry->GetInitializer();

		const FString Row = FString::Printf(TEXT("%s,%.3f,%d,%d,%d\n"),
			!Initializer.DebugName.IsNone() ? *Initializer.DebugName.ToString() : TEXT("*UNKNOWN*"),
			RegisteredGeometry.Size / double(1 << 20),
			Initializer.TotalPrimitiveCount,
			Initializer.Segments.Num(),
			(int32)Initializer.bAllowUpdate);
		CSVFile->Serialize(TCHAR_TO_ANSI(*Row), Row.Len());
	}

	return true;
}

void FRayTracingGeometryManager::DumpUnreferencedAlwaysResidentRayTracingGeometries() const
{
	TArray<RayTracing::FGeometryHandle> UnreferencedAlwaysResidentGeometries;

	for (RayTracing::FGeometryHandle GeometryHandle : AlwaysResidentGeometries)
	{
		const FRegisteredGeometry& RegisteredGeometry = GetRegisteredGeometry(GeometryHandle);
		const RayTracing::FGeometryGroupHandle GroupHandle = RegisteredGeometry.Geometry->GroupHandle;

		bool bIsReferenced = false;

		if (ReferencedGeometries.Contains(GeometryHandle))
		{
			bIsReferenced = true;
		}

		if (!GroupHandle.IsNull() && ReferencedGeometryGroups.Contains(GroupHandle))
		{
			bIsReferenced = true;
		}

		if (!bIsReferenced)
		{
			UnreferencedAlwaysResidentGeometries.Add(GeometryHandle);
		}
	}

	FString Filename = FPaths::ProfilingDir() + FString::Printf(TEXT("UnreferencedAlwaysResidentGeometries(%s).csv"), *FDateTime::Now().ToString(TEXT("%Y%m%d_%H%M%S")));

	if (DumpRayTracingGeometries(UnreferencedAlwaysResidentGeometries, Filename))
	{
		UE_LOGF(LogRayTracingGeometryManager, Log, "List of unreferenced always-resident ray tracing geometries written to file '%ls'", *Filename);
	}
	else
	{
		UE_LOGF(LogRayTracingGeometryManager, Log, "Unable to write list of unreferenced always-resident ray tracing geometries to file '%ls'", *Filename);
	}
}
#endif // !UE_BUILD_SHIPPING

#endif // RHI_RAYTRACING
