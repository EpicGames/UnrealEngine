// Copyright Epic Games, Inc. All Rights Reserved.

#include "Rendering/SkeletalMeshAttributeVertexBuffer.h"

#include "Rendering/SkeletalMeshVertexBuffer.h"


class FSkeletalMeshVertexAttributeData :
	public FSkeletalMeshVertexDataInterface,
	public TResourceArray<uint8, VERTEXBUFFER_ALIGNMENT>
{
	using FResourceArray = TResourceArray<uint8, VERTEXBUFFER_ALIGNMENT>;
public:
	explicit FSkeletalMeshVertexAttributeData(
		const int32 InStride,
		const bool InNeedsCPUAccess = false
		) :
		FResourceArray(InNeedsCPUAccess),
		Stride(InStride)
	{
	}
	
	// FSkeletalMeshVertexDataInterface implementations
	virtual void ResizeBuffer(uint32 InNumVertices) override
	{
		const int32 Count = static_cast<int32>(InNumVertices) * Stride;
		
		if (Num() < Count)
		{
			Reserve(Count);
			AddUninitialized(Count - Num());
		}
		else if (Num() > Count)
		{
			RemoveAt(Count, Num() - Count);
		}
	}

	virtual uint32 GetStride() const override
	{
		return Stride;
	}

	virtual uint8* GetDataPointer() override
	{
		return GetData();
	}
	

	virtual uint32 GetNumVertices() override
	{
		return Num() / Stride;
	}
	
	virtual FResourceArrayInterface* GetResourceArray() override
	{
		return this;
	}

	virtual void Serialize(FArchive& Ar) override
	{
		Ar << Stride;
		BulkSerialize(Ar);
	}

private:
	int32 Stride;
};


FSkeletalMeshAttributeVertexBuffer::~FSkeletalMeshAttributeVertexBuffer()
{
	CleanUp();
}


void FSkeletalMeshAttributeVertexBuffer::Init(
	const EPixelFormat InPixelFormat, 
	const int32 InNumVertices,
	const int32 InComponentCount,
	TConstArrayView<float> InValues
	)
{
	check(InNumVertices * InComponentCount == InValues.Num());
	check(InComponentCount >= 1 && InComponentCount <= 4);

	ComponentCount = InComponentCount;
	PixelFormat = InPixelFormat;

	Allocate();

	ValueData->ResizeBuffer(InNumVertices);

	SetData(InValues);
}

void FSkeletalMeshAttributeVertexBuffer::Allocate()
{
	CleanUp();

	switch(PixelFormat)
	{
	case PF_R8:
		ComponentStride = 1;
		break;
	case PF_R16F:
		ComponentStride = 2;
		break;
	case PF_R32_FLOAT:
		ComponentStride = 4;
		break;
	default:
		checkNoEntry();
		return;
	}

	ValueData = new FSkeletalMeshVertexAttributeData(ComponentStride * ComponentCount);
}


/** Convenience helpers to convert the float value data to various target types */
template<typename T> T ConvertValue(const float In);
template<> uint8 ConvertValue(const float In) { return FMath::Clamp(In, 0.0f, 1.0f) * MAX_uint8; }
template<> int8 ConvertValue(const float In) { return FMath::Clamp(In, -1.0f, 1.0f) * MAX_int8; }
template<> uint16 ConvertValue(const float In) { return FMath::Clamp(In, 0.0f, 1.0f) * MAX_uint16; }
template<> int16 ConvertValue(const float In) { return FMath::Clamp(In, -1.0f, 1.0f) * MAX_int16; }
template<> FFloat16 ConvertValue(const float In) { return FFloat16(In); }

template<typename T>
static void ConvertBatch(uint8* InDstValuesRawPtr, TConstArrayView<float> InSrcValues)
{
	T* DstValues = reinterpret_cast<T*>(InDstValuesRawPtr);
	for (int32 Index = 0; Index < InSrcValues.Num(); Index++)
	{
		*DstValues++ = ConvertValue<T>(InSrcValues[Index]);
	}
}


void FSkeletalMeshAttributeVertexBuffer::SetData(TConstArrayView<float> InValues)
{
	uint8* DstValuePtr = ValueData->GetDataPointer();

	switch (PixelFormat)
	{
	case PF_R8:
		ConvertBatch<uint8>(DstValuePtr, InValues);
		break;
	case PF_R16F:
		ConvertBatch<FFloat16>(DstValuePtr, InValues);
		break;
	case PF_R32_FLOAT:
		FMemory::Memcpy(DstValuePtr, InValues.GetData(), InValues.Num() * sizeof(float));
		break;
	default:
		checkNoEntry();
	}
}


void FSkeletalMeshAttributeVertexBuffer::CleanUp()
{
	delete ValueData;
	ValueData = nullptr;
}


int32 FSkeletalMeshAttributeVertexBuffer::GetResourceSize() const
{
	return ValueData ? ValueData->GetNumVertices() * ValueData->GetStride() : 0;
}


void FSkeletalMeshAttributeVertexBuffer::InitRHI(FRHICommandListBase& RHICmdList)
{
	FResourceArrayInterface* ResourceArray = ValueData->GetResourceArray();

	const FRHIBufferCreateDesc CreateDesc =
		FRHIBufferCreateDesc::CreateVertex(TEXT("SkeletalMeshAttributeVertexBuffer"), ResourceArray->GetResourceDataSize())
		.AddUsage(EBufferUsageFlags::Static | EBufferUsageFlags::ShaderResource)
		.SetInitActionResourceArray(ResourceArray)
		.DetermineInitialState();

	ValueBuffer.VertexBufferRHI = RHICmdList.CreateBuffer(CreateDesc);
	ValueBuffer.VertexBufferSRV = RHICmdList.CreateShaderResourceView(
		ValueBuffer.VertexBufferRHI,
		FRHIViewDesc::CreateBufferSRV()
			.SetType(FRHIViewDesc::EBufferType::Typed)
			.SetFormat(static_cast<EPixelFormat>(PixelFormat)));
}


void FSkeletalMeshAttributeVertexBuffer::ReleaseRHI()
{
	ValueBuffer.SafeRelease();
}


void FSkeletalMeshAttributeVertexBuffer::Serialize(FArchive& Ar)
{
	Ar << ComponentCount;
	Ar << PixelFormat;
	Ar << ComponentStride;

	if (Ar.IsLoading())
	{
		bool bValid;
		switch (PixelFormat)
		{
		case PF_R8:
			bValid = ComponentStride == 1;
			break;
		case PF_R16F:
			bValid = ComponentStride == 2;
			break;
		case PF_R32_FLOAT:
			bValid = ComponentStride == 4;
			break;
		default:
			bValid = false;
			break;
		}

		if (!bValid)
		{
			Ar.SetError();
			return;
		}

		Allocate();
	}

	if (ValueData)
	{
		ValueData->Serialize(Ar);
	}
}
