// Copyright Epic Games, Inc. All Rights Reserved.

#include "Rendering/SkeletalMeshHalfEdgeBuffer.h"

#include "RHIResourceReplace.h"
#include "Rendering/SkeletalMeshLODRenderData.h"

struct FEdgeKey
{
	FVector3f V0;
	FVector3f V1;

	bool operator==(FEdgeKey const& Rhs) const { return V0 == Rhs.V0 && V1 == Rhs.V1; }
};

static uint32 GetTypeHash(FEdgeKey const& Key)
{
	return HashCombine(GetTypeHash(Key.V0), GetTypeHash(Key.V1));
}

namespace
{
	/** Builds lookup from edge to twin edge. */
	class FTwinEdgeBuilder
	{
	public:
		FTwinEdgeBuilder(int32 InNumTriangles)
		{
			TwinEdges.SetNum(InNumTriangles * 3);
			for (int32& TwinEdge : TwinEdges)
			{
				TwinEdge = -1;
			}
		}

		int32 SmallestVector(FVector3f V0, FVector3f V1)
		{
			if (V0.X != V1.X) return V1.X > V0.X;
			if (V0.Y != V1.Y) return V1.Y > V0.Y;
			return V1.Z > V0.Z;
		}

		void Add(int32 InTriangleIndex, TStaticArray<FVector3f, 3>& InVertexPositions)
		{
			for (uint32 TriangleEdgeIndex = 0; TriangleEdgeIndex < 3; ++TriangleEdgeIndex)
			{
				FVector3f V[2] = { 
					InVertexPositions[TriangleEdgeIndex],
					InVertexPositions[(TriangleEdgeIndex + 1) % 3] };
				int32 VMin = SmallestVector(V[0], V[1]);
				const FEdgeKey Key({V[VMin], V[1 - VMin]});

				if (FEdgeDescription* EdgeDesc = EdgeMap.Find(Key))
				{
					if (EdgeDesc->E1 == -1)
					{
						// The second time that we've seen this edge.
						EdgeDesc->E1 = InTriangleIndex * 3 + TriangleEdgeIndex;
						TwinEdges[EdgeDesc->E0] = EdgeDesc->E1;
						TwinEdges[EdgeDesc->E1] = EdgeDesc->E0;
					}
					else
					{
						// Non-manifold geometry can end up here. Two options: 
						// (i) Don't store twins for this edge the first pair.
						// (ii) Reset the edge map entry and hope that good twins are met consecutively when walking the index buffer.
 						EdgeDesc->E0 = InTriangleIndex * 3 + TriangleEdgeIndex;
 						EdgeDesc->E1 = -1;
					}
				}
				else
				{
					// First time that we've seen this edge.
					EdgeMap.Add(Key).E0 = InTriangleIndex * 3 + TriangleEdgeIndex;
				}
			}
		}

		TResourceArray<int32>& GetTwinEdges()
		{
			return TwinEdges;
		}

	private:
		struct FEdgeDescription
		{
			int32 E0 = -1;
			int32 E1 = -1;
		};

		TMap<FEdgeKey, FEdgeDescription> EdgeMap;

		TResourceArray<int32> TwinEdges;
	};

	/** Builds lookup from (one) vertex to (many) edges. */
	class FVertexToEdgeBuilder
	{
	public:
		FVertexToEdgeBuilder(int32 InNumVertices, int32 InNumTriangles)
		{
			VertexToEdgeItems.Reserve(InNumTriangles * 3);
			VertexToEdgeItems.AddDefaulted(InNumVertices);
		}

		void Add(int32 InVertexIndex, int32 InEdgeIndex)
		{
			int32 ItemIndex = InVertexIndex;
			if (VertexToEdgeItems[ItemIndex].EdgeIndex == -1)
			{
				VertexToEdgeItems[ItemIndex].EdgeIndex = InEdgeIndex;
				return;
			}
			while (VertexToEdgeItems[ItemIndex].NextItem != -1)
			{
				ItemIndex = VertexToEdgeItems[ItemIndex].NextItem;
			}
			int32 NewItemIndex = VertexToEdgeItems.AddDefaulted();
			VertexToEdgeItems[ItemIndex].NextItem = NewItemIndex;
			VertexToEdgeItems[NewItemIndex].EdgeIndex = InEdgeIndex;
		}

		using FEdgeInlineArray = TArray<int32, TInlineAllocator<16>>;
		void Get(int32 InVertexIndex, FEdgeInlineArray& OutEdgeIndices) const
		{
			int32 ItemIndex = InVertexIndex;
			while (ItemIndex != -1)
			{
				OutEdgeIndices.Add(VertexToEdgeItems[ItemIndex].EdgeIndex);
				ItemIndex = VertexToEdgeItems[ItemIndex].NextItem;
			}
		}

	private:
		struct FVertexToEdgeItem
		{
			int32 EdgeIndex = -1;
			int32 NextItem = -1;
		};

		/** First NumVertices entries are allocated up front as start of per vertex linked lists. */
		TArray<FVertexToEdgeItem> VertexToEdgeItems;
	};

	static int32 GetHeadEdgeForVertex(int32 InVertexIndex, FVertexToEdgeBuilder const& InVertexToEdge, TResourceArray<int32> const& InEdgeToTwinEdge)
	{
		FVertexToEdgeBuilder::FEdgeInlineArray EdgeIndicesScratchBuffer;
		InVertexToEdge.Get(InVertexIndex, EdgeIndicesScratchBuffer);

		// Iterate backwards around the edges until we find a border, or a non recognized edge (which may lead to a border).
		const int32 StartEdgeIndex = EdgeIndicesScratchBuffer[0];
		int32 EdgeIndex = StartEdgeIndex;
		while(1)
		{
			const int32 LastEdgeIndex = EdgeIndex;
			EdgeIndex = ((EdgeIndex / 3) * 3) + ((EdgeIndex + 2) % 3);
			EdgeIndex = InEdgeToTwinEdge[EdgeIndex];
			
			if (EdgeIndex == StartEdgeIndex)
			{
				break;
			}
			if (EdgeIndex == -1 || EdgeIndicesScratchBuffer.Find(EdgeIndex) == INDEX_NONE)
			{
				EdgeIndex = LastEdgeIndex;
				break;
			}
		}

		return EdgeIndex;
	}
}

void SkeletalMeshHalfEdgeUtility::BuildHalfEdgeBuffers(const FSkeletalMeshLODRenderData& InLodRenderData, TResourceArray<int32>& OutVertexToEdge, TResourceArray<int32>& OutEdgeToTwinEdge)
{
	const FRawStaticIndexBuffer16or32Interface* IndexBuffer = InLodRenderData.MultiSizeIndexContainer.GetIndexBuffer();
	const int32 IndexCount = IndexBuffer->Num();
	const int32 TriangleCount = IndexCount / 3;

	const FPositionVertexBuffer& VertexBuffer = InLodRenderData.StaticVertexBuffers.PositionVertexBuffer;
	const int32 VertexCount = InLodRenderData.StaticVertexBuffers.PositionVertexBuffer.GetNumVertices();

	// Build edge to twin edge map.
	{
		FTwinEdgeBuilder TwinEdgeBuilder(TriangleCount);
		for (int32 TriangleIndex = 0; TriangleIndex < TriangleCount; ++TriangleIndex)
		{
			TStaticArray<FVector3f, 3> VertexPositions;
			VertexPositions[0] = VertexBuffer.VertexPosition(IndexBuffer->Get(TriangleIndex * 3 + 0));
			VertexPositions[1] = VertexBuffer.VertexPosition(IndexBuffer->Get(TriangleIndex * 3 + 1));
			VertexPositions[2] = VertexBuffer.VertexPosition(IndexBuffer->Get(TriangleIndex * 3 + 2));
			TwinEdgeBuilder.Add(TriangleIndex, VertexPositions);
		}
		OutEdgeToTwinEdge = MoveTemp(TwinEdgeBuilder.GetTwinEdges());
	}

	// Build vertex to edge index map.
	FVertexToEdgeBuilder VertexToEdgeBuilder(VertexCount, TriangleCount);
	for (int32 Index = 0; Index < TriangleCount * 3; ++Index)
	{
		const int32 VertexIndex = IndexBuffer->Get(Index);
		const int32 EdgeIndex = Index;
		VertexToEdgeBuilder.Add(VertexIndex, EdgeIndex);
	}

	// VertexToEdgeBuilder gives us multiple edges per vertex.
	// These should form a linked list of edges. 
	// Usually that list will be a loop, but at border geometry there will be a head and end to the list.
	// For loops it doesn't matter where we start iteration. But at borders we want to use the head of the linked list to start iteration.
	// Non-manifold geometry will be strange, and potentially have multiple edge independent loops. We just do the best we can in that case.
	OutVertexToEdge.SetNumUninitialized(VertexCount);
	for (int32 VertexIndex = 0; VertexIndex < VertexCount; ++VertexIndex)
	{
		OutVertexToEdge[VertexIndex] = GetHeadEdgeForVertex(VertexIndex, VertexToEdgeBuilder, OutEdgeToTwinEdge);
	}	
}

FSkeletalMeshHalfEdgeBuffer::FSkeletalMeshHalfEdgeBuffer() = default;
FSkeletalMeshHalfEdgeBuffer::~FSkeletalMeshHalfEdgeBuffer() = default;

void FSkeletalMeshHalfEdgeBuffer::Init(const FSkeletalMeshLODRenderData& InLodRenderData)
{
	check(VertexToEdgeData.Num() == 0);
	SkeletalMeshHalfEdgeUtility::BuildHalfEdgeBuffers(InLodRenderData, VertexToEdgeData, EdgeToTwinEdgeData);	
}

FSkeletalMeshHalfEdgeBuffer::FRHIInfo FSkeletalMeshHalfEdgeBuffer::CreateRHIBuffer(FRHICommandListBase& RHICmdList)
{
	FRHIInfo Buffers;
	
	const static FLazyName ClassName(TEXT("FSkeletalMeshHalfEdgeBuffer"));

	uint32 MinBufferSize = GetMinBufferSize();
	{
		const uint32 BufferSize = VertexToEdgeData.GetResourceDataSize();

		FRHIBufferCreateDesc CreateDesc =
			FRHIBufferCreateDesc::CreateStructured(TEXT("VertexToEdgeData"), FMath::Max(BufferSize, MinBufferSize), MinBufferSize)
			.AddUsage(EBufferUsageFlags::Static | EBufferUsageFlags::ShaderResource)
			.SetInitialState(ERHIAccess::SRVMask)
			.SetClassName(ClassName)
			.SetOwnerName(GetOwnerName());

		if (BufferSize > 0)
		{
			CreateDesc.SetInitActionResourceArray(&VertexToEdgeData);
		}

		Buffers.VertexToEdgeBufferRHI = RHICmdList.CreateBuffer(CreateDesc);
	}

	{
		const uint32 BufferSize = EdgeToTwinEdgeData.GetResourceDataSize();

		FRHIBufferCreateDesc CreateDesc =
			FRHIBufferCreateDesc::CreateStructured(TEXT("EdgeToTwinEdgeData"), FMath::Max(BufferSize, MinBufferSize), sizeof(int32))
			.AddUsage(EBufferUsageFlags::Static | EBufferUsageFlags::ShaderResource)
			.SetInitialState(ERHIAccess::SRVMask)
			.SetClassName(ClassName)
			.SetOwnerName(GetOwnerName());

		if (BufferSize > 0)
		{
			CreateDesc.SetInitActionResourceArray(&EdgeToTwinEdgeData);
		}

		Buffers.EdgeToTwinEdgeBufferRHI = RHICmdList.CreateBuffer(CreateDesc);
	}

	return Buffers;
}

void FSkeletalMeshHalfEdgeBuffer::InitRHIForStreaming(FRHIInfo RHIInfo, FRHIResourceReplaceBatcher& Batcher)
{
	if (VertexToEdgeBufferRHI && RHIInfo.VertexToEdgeBufferRHI)
	{
		Batcher.EnqueueReplace(VertexToEdgeBufferRHI, RHIInfo.VertexToEdgeBufferRHI);
	}
	if (EdgeToTwinEdgeBufferRHI && RHIInfo.EdgeToTwinEdgeBufferRHI)
	{
		Batcher.EnqueueReplace(EdgeToTwinEdgeBufferRHI, RHIInfo.EdgeToTwinEdgeBufferRHI);
	}
}

void FSkeletalMeshHalfEdgeBuffer::ReleaseRHIForStreaming(FRHIResourceReplaceBatcher& Batcher)
{
	if (VertexToEdgeBufferRHI)
	{
		Batcher.EnqueueReplace(VertexToEdgeBufferRHI, nullptr);
	}
	if (EdgeToTwinEdgeBufferRHI )
	{
		Batcher.EnqueueReplace(EdgeToTwinEdgeBufferRHI, nullptr);
	}	
}

void FSkeletalMeshHalfEdgeBuffer::InitRHI(FRHICommandListBase& RHICmdList)
{
	FRHIInfo Buffers = CreateRHIBuffer(RHICmdList);
	
	VertexToEdgeBufferRHI = Buffers.VertexToEdgeBufferRHI;
	EdgeToTwinEdgeBufferRHI = Buffers.EdgeToTwinEdgeBufferRHI;
	// We should always have a RHI and thus a SRV, even when CPU data is empty,
	// which can happen in two cases:
	// 1. It was not cooked with the skeletal mesh, in this case the buffer is never used.
	// 2. It has not been streamed in for this LOD yet. But once streamed, 
	//    the RHI will takeover ownership of the streamed buffer and becomes usable
	if (ensure(VertexToEdgeBufferRHI))
	{
		VertexToEdgeBufferSRV = RHICmdList.CreateShaderResourceView(VertexToEdgeBufferRHI, FRHIViewDesc::CreateBufferSRV().SetTypeFromBuffer(VertexToEdgeBufferRHI));
	}
	if (ensure(EdgeToTwinEdgeBufferRHI))
	{
		EdgeToTwinEdgeBufferSRV = RHICmdList.CreateShaderResourceView(EdgeToTwinEdgeBufferRHI, FRHIViewDesc::CreateBufferSRV().SetTypeFromBuffer(EdgeToTwinEdgeBufferRHI));
	}
}

void FSkeletalMeshHalfEdgeBuffer::ReleaseRHI()
{
	VertexToEdgeBufferRHI.SafeRelease();
	VertexToEdgeBufferSRV.SafeRelease();
	EdgeToTwinEdgeBufferRHI.SafeRelease();
	EdgeToTwinEdgeBufferSRV.SafeRelease();
}

bool FSkeletalMeshHalfEdgeBuffer::IsCPUDataValid() const
{
	return VertexToEdgeData.Num() > 0 && EdgeToTwinEdgeData.Num() > 0;
}

bool FSkeletalMeshHalfEdgeBuffer::IsReadyForRendering() const
{
	// The size of a buffer with valid data is definitely larger than the min buffer size
	// Buffer not being ready does not always mean that the data was not cooked,
	// it could also mean that the data hasn't been streamed in yet
	return IsInitialized() && VertexToEdgeBufferRHI->GetSize() > GetMinBufferSize();
}

void FSkeletalMeshHalfEdgeBuffer::CleanUp()
{
	VertexToEdgeData.Reset();
	EdgeToTwinEdgeData.Reset();	
}

int32 FSkeletalMeshHalfEdgeBuffer::GetResourceSize() const
{
	return VertexToEdgeData.GetResourceDataSize() + EdgeToTwinEdgeData.GetResourceDataSize();
}

void FSkeletalMeshHalfEdgeBuffer::Serialize(FArchive& Ar)
{
	Ar << VertexToEdgeData;
	Ar << EdgeToTwinEdgeData;
}

uint32 FSkeletalMeshHalfEdgeBuffer::GetMinBufferSize() const
{
	return sizeof(int32);
}

FArchive& operator<<(FArchive& Ar, FSkeletalMeshHalfEdgeBuffer& HalfEdgeBuffer)
{
	HalfEdgeBuffer.Serialize(Ar);
	return Ar;
}
