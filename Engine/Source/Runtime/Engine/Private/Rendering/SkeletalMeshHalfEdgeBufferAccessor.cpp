// Copyright Epic Games, Inc. All Rights Reserved.

#include "Rendering/SkeletalMeshHalfEdgeBufferAccessor.h"

#include "AssetRegistry/AssetRegistryModule.h"
#include "AssetRegistry/AssetData.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SkeletalMeshHalfEdgeBufferAccessor)

DEFINE_LOG_CATEGORY_STATIC(LogSkeletalMeshHalfEdgeBufferAccessor, Log, All);

FName SkeletalMeshHalfEdgeBufferAccessor::GetHalfEdgeRequirementAssetTagName()
{
	static FName Tag("bRequiresSkeletalMeshHalfEdgeBuffer");
	return Tag;
}

bool SkeletalMeshHalfEdgeBufferAccessor::IsHalfEdgeRequired(TSoftObjectPtr<UObject> InAssetSoftPtr)
{
	if (InAssetSoftPtr.IsNull())
	{
		return false;
	}
	
	IAssetRegistry& AssetRegistry = FModuleManager::LoadModuleChecked<FAssetRegistryModule>("AssetRegistry").Get();
	
	// Always ask for scan in case no initial scan was done (Commandlets) or if AssetRegistry isn't done scanning.
	AssetRegistry.ScanSynchronous({ }, { InAssetSoftPtr.GetLongPackageName()});
		
	FAssetData AssetData;
	AssetRegistry.TryGetAssetByObjectPath(InAssetSoftPtr.ToSoftObjectPath(), AssetData);

	// Asset types that implement USkeletalMeshHalfEdgeBufferAccessor should have this tag,
	// if not, it is likely old and requires a resave to have this information saved as asset tag
	bool bIsHalfEdgeRequired = false;
	if (AssetData.GetTagValue<bool>(GetHalfEdgeRequirementAssetTagName(), bIsHalfEdgeRequired))
	{
		return bIsHalfEdgeRequired;
	}
	
	// Fallback path if input is an old deformer graph asset
	//
	// If an asset do not have this tag serialized is likely old and haven't been re-saved recently.
	// Therefore, we have to simply assume that the deformer needs half-edge buffer here to make sure it still works,
	// while giving out a warning to ask for manual resave to avoid building half edge buffers unnecessarily.
	//
	// Ideally we want to load the asset and check it directly but loading isn't safe here
	// because this function can be called from worker threads during load/build.
	
	UE_LOGF(LogSkeletalMeshHalfEdgeBufferAccessor, Warning,
		"Unable to determine if Skeletal Mesh Half Edge data is required for Asset %ls, default to required. "
	   "Resaving the asset may help avoid building half edge data unnecessarily", *InAssetSoftPtr.ToString());
	
	return true;

}
