// Copyright Epic Games, Inc. All Rights Reserved.

#include "Rendering/SkeletalMeshLODImporterData.h"

#if WITH_EDITOR

#include "Algo/AnyOf.h"
#include "Engine/SkeletalMesh.h"
#include "Engine/SkinnedAssetCommon.h"
#include "ImportUtils/SkeletalMeshImportUtils.h"
#include "Materials/MaterialInterface.h"
#include "MeshElementContainer.h"
#include "Misc/ScopeRWLock.h"
#include "Misc/ScopedSlowTask.h"
#include "Modules/ModuleManager.h"
#include "Rendering/SkeletalMeshModel.h"
#include "Serialization/BulkDataReader.h"
#include "Serialization/BulkDataWriter.h"
#include "Serialization/LargeMemoryWriter.h"
#include "Serialization/MemoryReader.h"
#include "SkeletalMeshAttributes.h"
#include "SkeletalMeshOperations.h"

DEFINE_LOG_CATEGORY_STATIC(LogSkeletalMeshLODImporterData, Log, All);

void FSkeletalMeshImportData::CopyDataNeedByMorphTargetImport(FSkeletalMeshImportData& Other) const
{
	Other.Points = Points;
	//PointToRawMap should not be save when saving morph target data, we only need it temporary to gather the point from the fbx shape
	Other.PointToRawMap = PointToRawMap;

	Other.Faces = Faces;
	Other.bHasNormals = bHasNormals;
	Other.bHasTangents = bHasTangents;
}

void FSkeletalMeshImportData::KeepAlternateSkinningBuildDataOnly()
{
	//No need of any alternate restore data, since we are this data if this function is called
	AlternateInfluenceProfileNames.Empty();
	AlternateInfluences.Empty();
	
	//No need of the morph target restore data
	MorphTargetModifiedPoints.Empty();
	MorphTargetNames.Empty();
	MorphTargets.Empty();

	//Remove material array and PointToRawMap
	Materials.Empty();
	PointToRawMap.Empty();
	
	VertexAttributes.Empty();
	VertexAttributeNames.Empty();
}

/**
* Takes an imported bone name, removes any leading or trailing spaces, and converts the remaining spaces to dashes.
*/
FString FSkeletalMeshImportData::FixupBoneName(FString BoneName)
{
	BoneName.TrimStartAndEndInline();
	BoneName.ReplaceInline(TEXT(" "), TEXT("-"), ESearchCase::IgnoreCase);
	return BoneName;
}

/**
* Copy mesh data for importing a single LOD
*
* @param LODPoints - vertex data.
* @param LODWedges - wedge information to static LOD level.
* @param LODFaces - triangle/ face data to static LOD level.
* @param LODInfluences - weights/ influences to static LOD level.
*/
void FSkeletalMeshImportData::CopyLODImportData(
	TArray<FVector3f>& LODPoints,
	TArray<SkeletalMeshImportData::FMeshWedge>& LODWedges,
	TArray<SkeletalMeshImportData::FMeshFace>& LODFaces,
	TArray<SkeletalMeshImportData::FVertInfluence>& LODInfluences,
	TArray<int32>& LODPointToRawMap) const
{
	// Copy vertex data.
	LODPoints.Empty(Points.Num());
	LODPoints.AddUninitialized(Points.Num());
	for (int32 p = 0; p < Points.Num(); p++)
	{
		LODPoints[p] = Points[p];
	}

	// Copy wedge information to static LOD level.
	LODWedges.Empty(Wedges.Num());
	LODWedges.AddUninitialized(Wedges.Num());
	for (int32 w = 0; w < Wedges.Num(); w++)
	{
		LODWedges[w].iVertex = Wedges[w].VertexIndex;
		// Copy all texture coordinates
		FMemory::Memcpy(LODWedges[w].UVs, Wedges[w].UVs, sizeof(FVector2f) * MAX_TEXCOORDS); 
		LODWedges[w].Color = Wedges[w].Color;

	}

	// Copy triangle/ face data to static LOD level.
	LODFaces.Empty(Faces.Num());
	LODFaces.AddUninitialized(Faces.Num());
	for (int32 f = 0; f < Faces.Num(); f++)
	{
		SkeletalMeshImportData::FMeshFace Face;
		Face.iWedge[0] = Faces[f].WedgeIndex[0];
		Face.iWedge[1] = Faces[f].WedgeIndex[1];
		Face.iWedge[2] = Faces[f].WedgeIndex[2];
		Face.MeshMaterialIndex = Faces[f].MatIndex;

		Face.TangentX[0] = Faces[f].TangentX[0];
		Face.TangentX[1] = Faces[f].TangentX[1];
		Face.TangentX[2] = Faces[f].TangentX[2];

		Face.TangentY[0] = Faces[f].TangentY[0];
		Face.TangentY[1] = Faces[f].TangentY[1];
		Face.TangentY[2] = Faces[f].TangentY[2];

		Face.TangentZ[0] = Faces[f].TangentZ[0];
		Face.TangentZ[1] = Faces[f].TangentZ[1];
		Face.TangentZ[2] = Faces[f].TangentZ[2];

		Face.SmoothingGroups = Faces[f].SmoothingGroups;

		LODFaces[f] = Face;
	}

	// Copy weights/ influences to static LOD level.
	LODInfluences.Empty(Influences.Num());
	LODInfluences.AddUninitialized(Influences.Num());
	for (int32 i = 0; i < Influences.Num(); i++)
	{
		LODInfluences[i].Weight = Influences[i].Weight;
		LODInfluences[i].VertIndex = Influences[i].VertexIndex;
		LODInfluences[i].BoneIndex = Influences[i].BoneIndex;
	}

	// Copy mapping
	LODPointToRawMap = PointToRawMap;
}

bool FSkeletalMeshImportData::ReplaceSkeletalMeshGeometryImportData(const USkeletalMesh* SkeletalMesh, FSkeletalMeshImportData* ImportData, int32 LodIndex)
{
	FSkeletalMeshModel *ImportedResource = SkeletalMesh->GetImportedModel();
	check(ImportedResource && ImportedResource->LODModels.IsValidIndex(LodIndex));
	FSkeletalMeshLODModel& SkeletalMeshLODModel = ImportedResource->LODModels[LodIndex];

	const FSkeletalMeshLODInfo* LodInfo = SkeletalMesh->GetLODInfo(LodIndex);
	check(LodInfo);

	//Load the original skeletal mesh import data
	FSkeletalMeshImportData OriginalSkeletalMeshImportData;
	
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	SkeletalMesh->LoadLODImportedData(LodIndex, OriginalSkeletalMeshImportData);
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	//Backup the new geometry and rig to be able to apply the rig to the old geometry
	FSkeletalMeshImportData NewGeometryAndRigData = *ImportData;

	ImportData->bHasNormals = OriginalSkeletalMeshImportData.bHasNormals;
	ImportData->bHasTangents = OriginalSkeletalMeshImportData.bHasTangents;
	ImportData->bHasVertexColors = OriginalSkeletalMeshImportData.bHasVertexColors;
	ImportData->NumTexCoords = OriginalSkeletalMeshImportData.NumTexCoords;

	ImportData->Materials.Reset();
	ImportData->Points.Reset();
	ImportData->Faces.Reset();
	ImportData->Wedges.Reset();
	ImportData->PointToRawMap.Reset();
	ImportData->MorphTargetNames.Reset();
	ImportData->MorphTargets.Reset();
	ImportData->MorphTargetModifiedPoints.Reset();
	ImportData->MeshInfos.Reset();
	ImportData->VertexAttributes.Reset();
	ImportData->VertexAttributeNames.Reset();

	//Material is a special case since we cannot serialize the UMaterialInstance when saving the RawSkeletalMeshBulkData
	//So it has to be reconstructed.
	ImportData->MaxMaterialIndex = 0;
	const TArray<FSkeletalMaterial>& SkeletalMeshMaterials = SkeletalMesh->GetMaterials();
	for (int32 MaterialIndex = 0; MaterialIndex < SkeletalMeshMaterials.Num(); ++MaterialIndex)
	{
		SkeletalMeshImportData::FMaterial NewMaterial;

		NewMaterial.MaterialImportName = SkeletalMeshMaterials[MaterialIndex].ImportedMaterialSlotName.ToString();
		NewMaterial.Material = SkeletalMeshMaterials[MaterialIndex].MaterialInterface;
		// Add an entry for each unique material
		ImportData->MaxMaterialIndex = FMath::Max(ImportData->MaxMaterialIndex, (uint32)(ImportData->Materials.Add(NewMaterial)));
	}

	ImportData->NumTexCoords = OriginalSkeletalMeshImportData.NumTexCoords;
	ImportData->Points += OriginalSkeletalMeshImportData.Points;
	ImportData->Faces += OriginalSkeletalMeshImportData.Faces;
	ImportData->Wedges += OriginalSkeletalMeshImportData.Wedges;
	ImportData->PointToRawMap += OriginalSkeletalMeshImportData.PointToRawMap;
	ImportData->MorphTargetNames += OriginalSkeletalMeshImportData.MorphTargetNames;
	ImportData->MorphTargets += OriginalSkeletalMeshImportData.MorphTargets;
	ImportData->MorphTargetModifiedPoints += OriginalSkeletalMeshImportData.MorphTargetModifiedPoints;
	ImportData->MeshInfos += OriginalSkeletalMeshImportData.MeshInfos;
	ImportData->VertexAttributes += OriginalSkeletalMeshImportData.VertexAttributes;
	ImportData->VertexAttributeNames += OriginalSkeletalMeshImportData.VertexAttributeNames;

	return ImportData->ApplyRigToGeo(NewGeometryAndRigData);
}

bool FSkeletalMeshImportData::ReplaceSkeletalMeshRigImportData(const USkeletalMesh* SkeletalMesh, FSkeletalMeshImportData* ImportData, int32 LodIndex)
{
	FSkeletalMeshModel *ImportedResource = SkeletalMesh->GetImportedModel();
	check(ImportedResource && ImportedResource->LODModels.IsValidIndex(LodIndex));
	FSkeletalMeshLODModel& SkeletalMeshLODModel = ImportedResource->LODModels[LodIndex];

	const FSkeletalMeshLODInfo* LodInfo = SkeletalMesh->GetLODInfo(LodIndex);
	check(LodInfo);

	//Load the original skeletal mesh import data
	FSkeletalMeshImportData OriginalSkeletalMeshImportData;
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	SkeletalMesh->LoadLODImportedData(LodIndex, OriginalSkeletalMeshImportData);
PRAGMA_ENABLE_DEPRECATION_WARNINGS

	ImportData->RefBonesBinary.Reset();
	ImportData->RefBonesBinary += OriginalSkeletalMeshImportData.RefBonesBinary;

	//Fix the old rig to match the new geometry
	return ImportData->ApplyRigToGeo(OriginalSkeletalMeshImportData);
}

bool FSkeletalMeshImportData::ApplyRigToGeo(FSkeletalMeshImportData& Other)
{
	//Reset the influence, we will regenerate it from the other data (the incoming rig)
	Influences.Reset();

	FWedgePosition OldGeoOverlappingPosition;
	FWedgePosition::FillWedgePosition(OldGeoOverlappingPosition, Other.Points, Other.Wedges, Points, UE_THRESH_POINTS_ARE_SAME);
	FOctreeQueryHelper OctreeQueryHelper(OldGeoOverlappingPosition.GetOctree());

	//////////////////////////////////////////////////////////////////////////
	// Found the Remapping between old vertex index and new vertex index
	// The old vertex index are the key, the index of the first array
	// The new vertex indexes are the second array, because we can map many
	// new vertex to one old vertex
	//
	// All new wedges get remap to a old wedge index, so we can be sure that all
	// new vertex will have correct bone weight apply to them.
	TArray<TArray<int32>> OldToNewRemap;
	OldToNewRemap.AddDefaulted(Other.Points.Num());
	const int32 WedgeNum = Wedges.Num();
	int32 ProgressStep = FMath::Max<int32>(1, FMath::FloorToInt((float)WedgeNum/100.0f));
	int32 StepCount = FMath::CeilToInt((float)WedgeNum / (float)ProgressStep);

	FScopedSlowTask SlowTask((float)(StepCount), NSLOCTEXT("FSkeletalMeshImportData", "FSkeletalMeshImportData_ApplyRigToGeo_MainSlowTask", "Applying skinning to geometry..."));
	for (int32 WedgeIndex = 0; WedgeIndex < WedgeNum; ++WedgeIndex)
	{
		if (WedgeIndex % ProgressStep == 0)
		{
			SlowTask.EnterProgressFrame(1.0f);
		}
		const FVector2f& CurWedgeUV = Wedges[WedgeIndex].UVs[0];
		int32 NewVertexIndex = (int32)(Wedges[WedgeIndex].VertexIndex);
		FVector3f& NewPointA = Points[NewVertexIndex];
		SkeletalMeshImportData::FTriangle& NewFace = Faces[(WedgeIndex / 3)];
		int32 NewFaceCorner = (WedgeIndex % 3);
		FVector3f NewNormal = NewFace.TangentZ[NewFaceCorner];
		bool bFoundMatch = false;

		TArray<int32> OldWedgeIndexes;
		OldGeoOverlappingPosition.FindMatchingPositionWegdeIndexes(NewPointA, UE_THRESH_POINTS_ARE_SAME, OldWedgeIndexes);
		if (OldWedgeIndexes.Num() > 0)
		{
			//Getting the other 2 vertices of the new triangle
			FVector3f& NewPointB = Points[Wedges[NewFace.WedgeIndex[(WedgeIndex + 1) % 3]].VertexIndex];
			FVector3f& NewPointC = Points[Wedges[NewFace.WedgeIndex[(WedgeIndex + 2) % 3]].VertexIndex];
			int32 BestOldVertexIndex = INDEX_NONE;
			float LowestTriangleDeltaSum = 0;

			for (int32 OldWedgeIndex : OldWedgeIndexes)
			{
				int32 OldVertexIndex = Other.Wedges[OldWedgeIndex].VertexIndex;
				SkeletalMeshImportData::FTriangle& OldFace = Other.Faces[OldWedgeIndex / 3];
				int32 OldFaceCorner = (OldWedgeIndex % 3);
				FVector3f OldNormal = OldFace.TangentZ[OldFaceCorner];

				if (Other.Wedges[OldWedgeIndex].UVs[0].Equals(CurWedgeUV, UE_THRESH_UVS_ARE_SAME)
					&& OldNormal.Equals(NewNormal, UE_THRESH_NORMALS_ARE_SAME))
				{
					//If we have more than one good match, we select the vertex whose triangle is the most similar, 
					//that way we avoid picking the wrong vertex on a mirror mesh seam.
					if (OldWedgeIndexes.Num() == 1)
					{
						//We can skip the Delta calculation if there is only one similar vertex.
						BestOldVertexIndex = OldVertexIndex;
						break;
					}

					FVector3f& OldPointA = Other.Points[Other.Wedges[OldWedgeIndex].VertexIndex];
					FVector3f& OldPointB = Other.Points[Other.Wedges[OldFace.WedgeIndex[(OldWedgeIndex + 1) % 3]].VertexIndex];
					FVector3f& OldPointC = Other.Points[Other.Wedges[OldFace.WedgeIndex[(OldWedgeIndex + 2) % 3]].VertexIndex];
					float TriangleDeltaSum =
						(NewPointA - OldPointA).Size() +
						(NewPointB - OldPointB).Size() +
						(NewPointC - OldPointC).Size();

					if (BestOldVertexIndex == INDEX_NONE || TriangleDeltaSum < LowestTriangleDeltaSum)
					{
						BestOldVertexIndex = OldVertexIndex;
						LowestTriangleDeltaSum = TriangleDeltaSum;
					}
				}
			}

			if (BestOldVertexIndex != INDEX_NONE)
			{
				OldToNewRemap[BestOldVertexIndex].AddUnique(NewVertexIndex);
				bFoundMatch = true;
			}
		}

		//If some geometry was added, it will not found any exact match with the old geometry
		//In this case we have to find the nearest list of wedge indexes
		if(!bFoundMatch)
		{
			TArray<FWedgeInfo> NearestWedges;
			FVector3f SearchPosition = Points[NewVertexIndex];
			OctreeQueryHelper.FindNearestWedgeIndexes(SearchPosition, NearestWedges);
			//The best old wedge match is base on those weight ratio
			const int32 UVWeightRatioIndex = 0;
			const int32 NormalWeightRatioIndex = 1;
			const float MatchWeightRatio[3] = { 0.99f, 0.01f };
			if (NearestWedges.Num() > 0)
			{
				int32 BestOldVertexIndex = INDEX_NONE;
				float MaxUVDistance = 0.0f;
				float MaxNormalDelta = 0.0f;
				TArray<float> UvDistances;
				UvDistances.Reserve(NearestWedges.Num());
				TArray<float> NormalDeltas;
				NormalDeltas.Reserve(NearestWedges.Num());
				for (const FWedgeInfo& WedgeInfo : NearestWedges)
				{
					int32 OldWedgeIndex = WedgeInfo.WedgeIndex;
					int32 OldVertexIndex = Other.Wedges[OldWedgeIndex].VertexIndex;
					int32 OldFaceIndex = (OldWedgeIndex / 3);
					int32 OldFaceCorner = (OldWedgeIndex % 3);
					const FVector2f& OldUV = Other.Wedges[OldWedgeIndex].UVs[0];
					const FVector3f& OldNormal = Other.Faces[OldFaceIndex].TangentZ[OldFaceCorner];
					float UVDelta = FVector2f::DistSquared(CurWedgeUV, OldUV);
					float NormalDelta = FMath::Abs(FMath::Acos(FVector3f::DotProduct(NewNormal, OldNormal)));
					if (UVDelta > MaxUVDistance)
					{
						MaxUVDistance = UVDelta;
					}
					UvDistances.Add(UVDelta);
					if (NormalDelta > MaxNormalDelta)
					{
						MaxNormalDelta = NormalDelta;
					}
					NormalDeltas.Add(NormalDelta);
				}
				float BestContribution = 0.0f;
				for (int32 NearestWedgeIndex = 0; NearestWedgeIndex < UvDistances.Num(); ++NearestWedgeIndex)
				{
					float Contribution = ((MaxUVDistance - UvDistances[NearestWedgeIndex])/MaxUVDistance)*MatchWeightRatio[UVWeightRatioIndex];
					Contribution += ((MaxNormalDelta - NormalDeltas[NearestWedgeIndex]) / MaxNormalDelta)*MatchWeightRatio[NormalWeightRatioIndex];
					if (Contribution > BestContribution)
					{
						BestContribution = Contribution;
						BestOldVertexIndex = Other.Wedges[NearestWedges[NearestWedgeIndex].WedgeIndex].VertexIndex;
					}
				}
				if (BestOldVertexIndex == INDEX_NONE)
				{
					//Use the first NearestWedges entry, we end up here because all NearestWedges entries all equals, so the ratio will be zero in such a case
					BestOldVertexIndex = Other.Wedges[NearestWedges[0].WedgeIndex].VertexIndex;
				}
				OldToNewRemap[BestOldVertexIndex].AddUnique(NewVertexIndex);
			}
		}
	}

	for (int32 InfluenceIndex = 0; InfluenceIndex < Other.Influences.Num(); ++InfluenceIndex)
	{
		int32 OldPointIndex = Other.Influences[InfluenceIndex].VertexIndex;

		const TArray<int32>& NewInfluenceVertexIndexes = OldToNewRemap[OldPointIndex];

		for (int32 NewPointIdx : NewInfluenceVertexIndexes)
		{
			SkeletalMeshImportData::FRawBoneInfluence& RawBoneInfluence = Influences.AddDefaulted_GetRef();
			RawBoneInfluence.BoneIndex = Other.Influences[InfluenceIndex].BoneIndex;
			RawBoneInfluence.Weight = Other.Influences[InfluenceIndex].Weight;
			RawBoneInfluence.VertexIndex = NewPointIdx;
		}
	}

	return true;
}

/**
* Serialization of raw meshes uses its own versioning scheme because it is
* stored in bulk data.
*/
enum
{
	// Engine raw mesh version:
	REDUCTION_BASE_SK_DATA_BULKDATA_VER_INITIAL = 0,
	
	//////////////////////////////////////////////////////////////////////////
	// Add new raw mesh versions here.

	REDUCTION_BASE_SK_DATA_BULKDATA_VER_PLUS_ONE,
	REDUCTION_BASE_SK_DATA_BULKDATA_VER = REDUCTION_BASE_SK_DATA_BULKDATA_VER_PLUS_ONE - 1,

	// Licensee raw mesh version:
	REDUCTION_BASE_SK_DATA_BULKDATA_LIC_VER_INITIAL = 0,
	
	//////////////////////////////////////////////////////////////////////////
	// Licensees add new raw mesh versions here.

	REDUCTION_BASE_SK_DATA_BULKDATA_LIC_VER_PLUS_ONE,
	REDUCTION_BASE_SK_DATA_BULKDATA_LIC_VER = REDUCTION_BASE_SK_DATA_BULKDATA_LIC_VER_PLUS_ONE - 1
};

struct FReductionSkeletalMeshData
{
	FReductionSkeletalMeshData(FSkeletalMeshLODModel& InBaseLODModel, TMap<FString, TArray<FMorphTargetDelta>>& InBaseLODMorphTargetData, UObject* InOwner)
		: BaseLODModel(InBaseLODModel)
		, BaseLODMorphTargetData(InBaseLODMorphTargetData)
		, Owner(InOwner)
	{
	}

	FSkeletalMeshLODModel& BaseLODModel;
	TMap<FString, TArray<FMorphTargetDelta>>& BaseLODMorphTargetData;
	UObject* Owner;
};

FArchive& operator<<(FArchive& Ar, FReductionSkeletalMeshData& ReductionSkeletalMeshData)
{
	int32 Version = REDUCTION_BASE_SK_DATA_BULKDATA_VER;
	int32 LicenseeVersion = REDUCTION_BASE_SK_DATA_BULKDATA_LIC_VER;
	Ar << Version;
	Ar << LicenseeVersion;
	ReductionSkeletalMeshData.BaseLODModel.Serialize(Ar, ReductionSkeletalMeshData.Owner, INDEX_NONE);
	
	if(Ar.IsLoading() && Ar.AtEnd())
	{
		//Hack to fix a serialization error, serialize the MorphTargetData only if there is some left space in the archive
		UE_ASSET_LOG(LogSkeletalMeshLODImporterData, Display, ReductionSkeletalMeshData.Owner, TEXT("This skeletalMesh should be re-import to save some missing reduction source data."));
	}
	else
	{
		Ar << ReductionSkeletalMeshData.BaseLODMorphTargetData;
	}
	return Ar;
}

FReductionBaseSkeletalMeshBulkData::FReductionBaseSkeletalMeshBulkData()
{
}

void FReductionBaseSkeletalMeshBulkData::Serialize(FArchive& Ar, TArray<FReductionBaseSkeletalMeshBulkData*>& ReductionBaseSkeletalMeshDatas, UObject* Owner)
{
	Ar.CountBytes(ReductionBaseSkeletalMeshDatas.Num() * sizeof(FReductionBaseSkeletalMeshBulkData), ReductionBaseSkeletalMeshDatas.Num() * sizeof(FReductionBaseSkeletalMeshBulkData));
	if (Ar.IsLoading())
	{
		// Load array.
		int32 NewNum;
		Ar << NewNum;
		ReductionBaseSkeletalMeshDatas.Empty(NewNum);
		for (int32 Index = 0; Index < NewNum; Index++)
		{
			FReductionBaseSkeletalMeshBulkData* EmptyData = new FReductionBaseSkeletalMeshBulkData();
			int32 NewEntryIndex = ReductionBaseSkeletalMeshDatas.Add(EmptyData);
			check(NewEntryIndex == Index);
			ReductionBaseSkeletalMeshDatas[Index]->Serialize(Ar, Owner);
		}
	}
	else
	{
		// Save array.
		int32 Num = ReductionBaseSkeletalMeshDatas.Num();
		Ar << Num;
		for (int32 Index = 0; Index < Num; Index++)
		{
			(ReductionBaseSkeletalMeshDatas)[Index]->Serialize(Ar, Owner);
		}
	}
}

void FReductionBaseSkeletalMeshBulkData::Serialize(FArchive& Ar, UObject* Owner)
{
	if (Ar.IsTransacting())
	{
		// If transacting, keep these members alive the other side of an undo, otherwise their values will get lost
		Ar << UEVersion;
		Ar << LicenseeUEVersion;
		SerializeLoadingCustomVersionContainer.Serialize(Ar);
		Ar << bUseSerializeLoadingCustomVersion;
	}
	else
	{
		if (Ar.IsSaving() && bUseSerializeLoadingCustomVersion == true)
		{
			//We need to update the FReductionSkeletalMeshData serialize version to the latest in case we save the Parent bulkdata
			FSkeletalMeshLODModel BaseLODModel;
			TMap<FString, TArray<FMorphTargetDelta>> BaseLODMorphTargetData;
			LoadReductionData(BaseLODModel, BaseLODMorphTargetData, Owner);
			SaveReductionData(BaseLODModel, BaseLODMorphTargetData, Owner);
		}
	}

	BulkData.Serialize(Ar, Owner);
	if (!Ar.IsTransacting() && Ar.IsLoading())
	{
		//Save the custom version so we can load FReductionSkeletalMeshData later
		BulkData.GetBulkDataVersions(Ar, UEVersion, LicenseeUEVersion, SerializeLoadingCustomVersionContainer);
		bUseSerializeLoadingCustomVersion = true;
	}
}

void FReductionBaseSkeletalMeshBulkData::SaveReductionData(FSkeletalMeshLODModel& BaseLODModel, TMap<FString, TArray<FMorphTargetDelta>>& BaseLODMorphTargetData, UObject* Owner)
{
	check(IsInGameThread());

	//Saving the bulk data mean we do not need anymore the SerializeLoadingCustomVersionContainer of the parent bulk data
	SerializeLoadingCustomVersionContainer.Empty();
	bUseSerializeLoadingCustomVersion = false;
	FReductionSkeletalMeshData ReductionSkeletalMeshData(BaseLODModel, BaseLODMorphTargetData, Owner);

	CacheGeometryInfo(BaseLODModel);

	//Clear the bulk data before writing it
	BulkData.RemoveBulkData();

	// Get a lock on the bulk data
	{
		const bool bIsPersistent = true;
		FBulkDataWriter Ar(BulkData, bIsPersistent);
		Ar << ReductionSkeletalMeshData;

		// Preserve CustomVersions at save time so we can reuse the same ones when reloading direct from memory
		UEVersion = Ar.UEVer();
		LicenseeUEVersion = Ar.LicenseeUEVer();
		SerializeLoadingCustomVersionContainer = Ar.GetCustomVersions();
	}
	// Unlock the bulk data
}

void FReductionBaseSkeletalMeshBulkData::LoadReductionData(FSkeletalMeshLODModel& BaseLODModel, TMap<FString, TArray<FMorphTargetDelta>>& BaseLODMorphTargetData, UObject* Owner)
{
	check(IsInGameThread() || IsInAsyncLoadingThread());

	BaseLODMorphTargetData.Empty();
	if (BulkData.GetElementCount() > 0)
	{
		FReductionSkeletalMeshData ReductionSkeletalMeshData(BaseLODModel, BaseLODMorphTargetData, Owner);

		// Get a lock on the bulk data
		{
			const bool bIsPersistent = true;
			FBulkDataReader Ar(BulkData, bIsPersistent);
			
			// Propagate the custom version information from the package to the bulk data, so that the MeshDescription
			// is serialized with the same versioning.
			Ar.SetUEVer(UEVersion);
			Ar.SetLicenseeUEVer(LicenseeUEVersion);
			Ar.SetCustomVersions(SerializeLoadingCustomVersionContainer);

			Ar << ReductionSkeletalMeshData;

			CacheGeometryInfo(BaseLODModel);

			//This call will filled missing chunked data for old asset that cannot build normal and chunking (not re import since the new skeletal mesh chunk build refactor)
			if (!BaseLODModel.bIsBuildDataAvailable)
			{
				BaseLODModel.UpdateChunkedSectionInfo(Owner ? Owner->GetName() : FString());
			}
		}
		// Unlock the bulk data
	}
}

void FReductionBaseSkeletalMeshBulkData::CacheGeometryInfo(const FSkeletalMeshLODModel& SourceLODModel)
{
	CacheLODVertexNumber = 0;
	CacheLODTriNumber = 0;
	for (int32 SectionIndex = 0; SectionIndex < SourceLODModel.Sections.Num(); ++SectionIndex)
	{
		const FSkelMeshSection& Section = SourceLODModel.Sections[SectionIndex];
		//We count disabled section, since the render buffer contain the disabled section data. This is crucial for memory budget
		//Make sure the count fit in a uint32
		CacheLODVertexNumber += Section.NumVertices < 0 ? 0 : Section.NumVertices;
		CacheLODTriNumber += Section.NumTriangles;
	}
}

void FReductionBaseSkeletalMeshBulkData::GetGeometryInfo(uint32& LODVertexNumber, uint32& LODTriNumber, UObject* Owner)
{
	if (!IsEmpty() && (CacheLODVertexNumber == MAX_uint32 || CacheLODTriNumber == MAX_uint32))
	{
		FSkeletalMeshLODModel ReductionSourceLODModel;
		TMap<FString, TArray<FMorphTargetDelta>> TempLODMorphTargetData;
		LoadReductionData(ReductionSourceLODModel, TempLODMorphTargetData, Owner);
		CacheGeometryInfo(ReductionSourceLODModel);
	}
	LODVertexNumber = CacheLODVertexNumber;
	LODTriNumber = CacheLODTriNumber;
}

/*------------------------------------------------------------------------------
FInlineReductionCacheData
------------------------------------------------------------------------------*/

void FInlineReductionCacheData::SetCacheGeometryInfo(const FSkeletalMeshLODModel& SourceLODModel)
{
	CacheLODVertexCount = 0;
	CacheLODTriCount = 0;
	for (int32 SectionIndex = 0; SectionIndex < SourceLODModel.Sections.Num(); ++SectionIndex)
	{
		const FSkelMeshSection& Section = SourceLODModel.Sections[SectionIndex];
		//We count disabled section, since the render buffer contain the disabled section data. This is crucial for memory budget
		//Make sure the count fit in a uint32
		CacheLODVertexCount += Section.NumVertices < 0 ? 0 : Section.NumVertices;
		CacheLODTriCount += Section.NumTriangles;
	}
}

void FInlineReductionCacheData::SetCacheGeometryInfo(uint32 LODVertexCount, uint32 LODTriCount)
{
	CacheLODVertexCount = LODVertexCount;
	CacheLODTriCount = LODTriCount;
}

void FInlineReductionCacheData::GetCacheGeometryInfo(uint32& LODVertexCount, uint32& LODTriCount) const
{
	LODVertexCount = CacheLODVertexCount;
	LODTriCount = CacheLODTriCount;
}

/*------------------------------------------------------------------------------
FRawSkeletalMeshBulkData
------------------------------------------------------------------------------*/

FRawSkeletalMeshBulkData::FRawSkeletalMeshBulkData()
	: bGuidIsHash(false)
{
	GeoImportVersion = ESkeletalMeshGeoImportVersions::Before_Versionning;
	SkinningImportVersion = ESkeletalMeshSkinningImportVersions::Before_Versionning;
}


/**
* Serialization of raw meshes uses its own versioning scheme because it is
* stored in bulk data.
*/
enum
{
	// Engine raw mesh version:
	RAW_SKELETAL_MESH_BULKDATA_VER_INITIAL = 0,
	RAW_SKELETAL_MESH_BULKDATA_VER_AlternateInfluence = 1,
	RAW_SKELETAL_MESH_BULKDATA_VER_RebuildSystem = 2,
	RAW_SKELETAL_MESH_BULKDATA_VER_CompressMorphTargetData = 3,
	RAW_SKELETAL_MESH_BULKDATA_VER_VertexAttributes = 4,
	RAW_SKELETAL_MESH_BULKDATA_VER_Keep_Sections_Separate = 5,
	// Add new raw mesh versions here.

	RAW_SKELETAL_MESH_BULKDATA_VER_PLUS_ONE,
	RAW_SKELETAL_MESH_BULKDATA_VER = RAW_SKELETAL_MESH_BULKDATA_VER_PLUS_ONE - 1,

	// Licensee raw mesh version:
	RAW_SKELETAL_MESH_BULKDATA_LIC_VER_INITIAL = 0,
	// Licensees add new raw mesh versions here.

	RAW_SKELETAL_MESH_BULKDATA_LIC_VER_PLUS_ONE,
	RAW_SKELETAL_MESH_BULKDATA_LIC_VER = RAW_SKELETAL_MESH_BULKDATA_LIC_VER_PLUS_ONE - 1
};

FArchive& operator<<(FArchive& Ar, FSkeletalMeshImportData& RawMesh)
{
	int32 Version = RAW_SKELETAL_MESH_BULKDATA_VER;
	int32 LicenseeVersion = RAW_SKELETAL_MESH_BULKDATA_LIC_VER;
	Ar << Version;
	Ar << LicenseeVersion;

	/**
	* Serialization should use the raw mesh version not the archive version.
	* Additionally, stick to serializing basic types and arrays of basic types.
	*/
	bool bDummyFlag1 = false, bDummyFlag2 = false;

	Ar << bDummyFlag1;
	Ar << RawMesh.bHasNormals;
	Ar << RawMesh.bHasTangents;
	Ar << RawMesh.bHasVertexColors;
	Ar << bDummyFlag2;
	Ar << RawMesh.MaxMaterialIndex;
	Ar << RawMesh.NumTexCoords;
	
	Ar << RawMesh.Faces;
	Ar << RawMesh.Influences;
	Ar << RawMesh.Materials;
	Ar << RawMesh.Points;
	Ar << RawMesh.PointToRawMap;
	Ar << RawMesh.RefBonesBinary;
	Ar << RawMesh.Wedges;
	
	//In the old version this processing was done after we save the asset
	//We now save it after the processing is done so for old version we do it here when loading
	if (Ar.IsLoading() && Version < RAW_SKELETAL_MESH_BULKDATA_VER_AlternateInfluence)
	{
		SkeletalMeshImportUtils::ProcessImportMeshInfluences(RawMesh, FString("Unknown")); // Not sure how to get owning mesh name at this point...
	}

	
	if (Version >= RAW_SKELETAL_MESH_BULKDATA_VER_RebuildSystem)
	{
		Ar << RawMesh.MorphTargets;
		Ar << RawMesh.MorphTargetModifiedPoints;
		Ar << RawMesh.MorphTargetNames;
		Ar << RawMesh.AlternateInfluences;
		Ar << RawMesh.AlternateInfluenceProfileNames;
	}
	else if (Ar.IsLoading())
	{
		RawMesh.MorphTargets.Empty();
		RawMesh.MorphTargetModifiedPoints.Empty();
		RawMesh.MorphTargetNames.Empty();
		RawMesh.AlternateInfluences.Empty();
		RawMesh.AlternateInfluenceProfileNames.Empty();
	}

	if (Version >= RAW_SKELETAL_MESH_BULKDATA_VER_VertexAttributes)
	{
		Ar << RawMesh.VertexAttributes;
		Ar << RawMesh.VertexAttributeNames;
	}
	else
	{
		RawMesh.VertexAttributes.Empty();
		RawMesh.VertexAttributeNames.Empty();
	}

	if (Ar.IsLoading() && Version < RAW_SKELETAL_MESH_BULKDATA_VER_CompressMorphTargetData)
	{
		if (RawMesh.MorphTargetModifiedPoints.Num() != 0)
		{
			//Compress the morph target data
			for (int32 MorphTargetIndex = 0; MorphTargetIndex < RawMesh.MorphTargets.Num(); ++MorphTargetIndex)
			{
				if (!RawMesh.MorphTargetModifiedPoints.IsValidIndex(MorphTargetIndex))
				{
					continue;
				}
				const TSet<uint32>& ModifiedPoints = RawMesh.MorphTargetModifiedPoints[MorphTargetIndex];
				FSkeletalMeshImportData& ToCompressShapeImportData = RawMesh.MorphTargets[MorphTargetIndex];
				TArray<FVector3f> CompressPoints;
				CompressPoints.Reserve(ToCompressShapeImportData.Points.Num());
				for (uint32 PointIndex : ModifiedPoints)
				{
					CompressPoints.Add(ToCompressShapeImportData.Points[PointIndex]);
				}
				ToCompressShapeImportData.Points = CompressPoints;
			}
		}
	}

	if (Version >= RAW_SKELETAL_MESH_BULKDATA_VER_Keep_Sections_Separate)
	{
		bool bDummyFlag3 = false;
		Ar << bDummyFlag3;
	}

	return Ar;
}

void FRawSkeletalMeshBulkData::Serialize(FArchive& Ar, TArray<TSharedRef<FRawSkeletalMeshBulkData>>& RawSkeltalMeshBulkDatas, UObject* Owner)
{
	Ar.CountBytes(RawSkeltalMeshBulkDatas.Num() * sizeof(FRawSkeletalMeshBulkData), RawSkeltalMeshBulkDatas.Num() * sizeof(FRawSkeletalMeshBulkData));
	if (Ar.IsLoading())
	{
		// Load array.
		int32 NewNum;
		Ar << NewNum;
		RawSkeltalMeshBulkDatas.Empty(NewNum);
		for (int32 Index = 0; Index < NewNum; Index++)
		{
			int32 NewEntryIndex = RawSkeltalMeshBulkDatas.Add(MakeShared<FRawSkeletalMeshBulkData>());
			check(NewEntryIndex == Index);
			RawSkeltalMeshBulkDatas[Index].Get().Serialize(Ar, Owner);
		}
	}
	else
	{
		// Save array.
		int32 Num = RawSkeltalMeshBulkDatas.Num();
		Ar << Num;
		for (int32 Index = 0; Index < Num; Index++)
		{
			RawSkeltalMeshBulkDatas[Index].Get().Serialize(Ar, Owner);
		}
	}
}

void FRawSkeletalMeshBulkData::Serialize(FArchive& Ar, UObject* Owner)
{
	if (Ar.IsTransacting())
	{
		// If transacting, keep these members alive the other side of an undo, otherwise their values will get lost
		Ar << UEVersion;
		Ar << LicenseeUEVersion;
		SerializeLoadingCustomVersionContainer.Serialize(Ar);
		Ar << bUseSerializeLoadingCustomVersion;
	}
	else
	{
		if (Ar.IsSaving() && bUseSerializeLoadingCustomVersion == true)
		{
			//We need to update the FReductionSkeletalMeshData serialize version to the latest in case we save the Parent bulkdata
			UpdateRawMeshFormat();
		}
	}

	Ar.UsingCustomVersion(FEditorObjectVersion::GUID);

	if (Ar.CustomVer(FEditorObjectVersion::GUID) >= FEditorObjectVersion::SkeletalMeshBuildRefactor)
	{
		Ar << GeoImportVersion;
		Ar << SkinningImportVersion;
	}
	else
	{
		GeoImportVersion = ESkeletalMeshGeoImportVersions::Before_Versionning;
		SkinningImportVersion = ESkeletalMeshSkinningImportVersions::Before_Versionning;
	}

	// An exclusive lock is required so we can safely load/save the raw data from multiple threads
	{
		FWriteScopeLock ScopeLock(BulkDataLock.Get());
		BulkData.Serialize(Ar, Owner);
	}

	Ar << Guid;
	Ar << bGuidIsHash;

	if (!Ar.IsTransacting() && Ar.IsLoading())
	{
		//Save the custom version so we can load FReductionSkeletalMeshData later
		BulkData.GetBulkDataVersions(Ar, UEVersion, LicenseeUEVersion, SerializeLoadingCustomVersionContainer);
		bUseSerializeLoadingCustomVersion = true;
	}
}

void FRawSkeletalMeshBulkData::SaveRawMesh(FSkeletalMeshImportData& InMesh)
{
	// An exclusive lock is required so we can safely load the raw data from multiple threads
	FWriteScopeLock ScopeLock(BulkDataLock.Get());
	
	//Saving the bulk data mean we do not need anymore the SerializeLoadingCustomVersionContainer of the parent bulk data
	SerializeLoadingCustomVersionContainer.Empty();

	//Clear the bulk data before writing it
	BulkData.RemoveBulkData();
	BulkData.StoreCompressedOnDisk(NAME_Zlib);

	// Get a lock on the bulk data
	{
		const bool bIsPersistent = true;
		FBulkDataWriter Ar(BulkData, bIsPersistent);
		Ar << InMesh;

		// Preserve CustomVersions at save time so we can reuse the same ones when reloading direct from memory
		UEVersion = Ar.UEVer();
		LicenseeUEVersion = Ar.LicenseeUEVer();
		SerializeLoadingCustomVersionContainer = Ar.GetCustomVersions();
	}

	//Create the guid from the content, this allow to use the data into the ddc key
	FSHA1 Sha;
	if (BulkData.GetBulkDataSize() > 0)
	{
		uint8* Buffer = (uint8*)BulkData.Lock(LOCK_READ_ONLY);
		Sha.Update(Buffer, BulkData.GetBulkDataSize());
		BulkData.Unlock();
	}
	Sha.Final();
	uint32 Hash[5];
	Sha.GetHash((uint8*)Hash);
	Guid = FGuid(Hash[0] ^ Hash[4], Hash[1], Hash[2], Hash[3]);
}

void FRawSkeletalMeshBulkData::LoadRawMesh(FSkeletalMeshImportData& OutMesh)
{
	OutMesh.Empty();
	if (BulkData.GetElementCount() > 0)
	{
		// An exclusive lock is required so we can safely load the raw data from multiple threads
		FWriteScopeLock ScopeLock(BulkDataLock.Get());

		// This allows any thread to be able to deserialize from the RawMesh directly
		// from disk so we can unload bulk data from memory.
		bool bHasBeenLoadedFromFileReader = false;
		if (!BulkData.IsBulkDataLoaded())
		{
			// This can't be called in -game mode because we're not allowed to load bulk data outside of EDL.
			bHasBeenLoadedFromFileReader = BulkData.LoadBulkDataWithFileReader();
		}
		// This is in a scope because the FBulkDataReader need to be destroyed in order
		// to unlock the BulkData and allow UnloadBulkData to actually do its job.
		{
			const bool bIsPersistent = true;
			FBulkDataReader Ar(BulkData, bIsPersistent);

			// Propagate the custom version information from the package to the bulk data, so that the MeshDescription
			// is serialized with the same versioning.
			Ar.SetUEVer(UEVersion);
			Ar.SetLicenseeUEVer(LicenseeUEVersion);
			Ar.SetCustomVersions(SerializeLoadingCustomVersionContainer);
			Ar << OutMesh;
		}

		// Throw away the bulk data allocation only in the case we can safely reload it from disk
		// and if BulkData.LoadBulkDataWithFileReader() is allowed to work from any thread.
		// This saves a significant amount of memory during map loading of Nanite Meshes.
		if (bHasBeenLoadedFromFileReader)
		{
			verify(BulkData.UnloadBulkData());
		}
	}
}


void FRawSkeletalMeshBulkData::UpdateRawMeshFormat()
{
	if (BulkData.GetElementCount() == 0)
	{
		return;
	}

	// An exclusive lock is required so we can safely load the raw data from multiple threads
	FWriteScopeLock ScopeLock(BulkDataLock.Get());
	// This allows any thread to be able to deserialize from the RawMesh directly
	// from disk so we can unload bulk data from memory.
	bool bHasBeenLoadedFromFileReader = false;
	if (!BulkData.IsBulkDataLoaded())
	{
		// This can't be called in -game mode because we're not allowed to load bulk data outside of EDL.
		bHasBeenLoadedFromFileReader = BulkData.LoadBulkDataWithFileReader();
	}

	bool bModified = false;
	uint64 NumBytes = 0;
	FLargeMemoryWriter NewBytes(BulkData.GetBulkDataSize(), true /* bIsPersistent */);
	FSkeletalMeshImportData MeshImportData;
	// This is in a scope because the BulkData needs to be unlocked for UnloadBulkData.
	{
		FMemoryView OldBytes(BulkData.Lock(LOCK_READ_ONLY), BulkData.GetBulkDataSize());
		ON_SCOPE_EXIT
		{
			BulkData.Unlock();
		};
		FMemoryReaderView Reader(OldBytes, true /* bIsPersistent */);

		// Propagate the custom version information from the package to the bulk data, so that the MeshDescription
		// is serialized with the same versioning.
		Reader.SetUEVer(UEVersion);
		Reader.SetLicenseeUEVer(LicenseeUEVersion);
		Reader.SetCustomVersions(SerializeLoadingCustomVersionContainer);
		Reader << MeshImportData;

		NewBytes << MeshImportData;
		NumBytes = static_cast<uint64>(NewBytes.TotalSize());
		bModified = NumBytes != OldBytes.GetSize() ||
			0 != FMemory::Memcmp(OldBytes.GetData(), NewBytes.GetData(), NumBytes);
	}

	// Throw away the bulk data allocation only in the case we can safely reload it from disk
	// and if BulkData.LoadBulkDataWithFileReader() is allowed to work from any thread.
	// This saves a significant amount of memory during map loading of Nanite Meshes.
	if (bHasBeenLoadedFromFileReader)
	{
		verify(BulkData.UnloadBulkData());
	}

	if (bModified)
	{
		//Clear the bulk data before writing it
		BulkData.RemoveBulkData();
		BulkData.StoreCompressedOnDisk(NAME_Zlib);
		{
			BulkData.Lock(LOCK_READ_WRITE);
			ON_SCOPE_EXIT
			{
				BulkData.Unlock();
			};
			void* Buffer = BulkData.Realloc(NumBytes);
			FMemory::Memcpy(Buffer, NewBytes.GetData(), NumBytes);
		}
		// Preserve CustomVersions at save time so we can reuse the same ones when reloading direct from memory
		SerializeLoadingCustomVersionContainer = NewBytes.GetCustomVersions();

		//Create the guid from the content, this allow to use the data into the ddc key
		FSHA1 Sha;
		if (NumBytes > 0)
		{
			Sha.Update(NewBytes.GetData(), NumBytes);
		}
		Sha.Final();
		uint32 Hash[5];
		Sha.GetHash((uint8*)Hash);
		Guid = FGuid(Hash[0] ^ Hash[4], Hash[1], Hash[2], Hash[3]);
	}
}

FString FRawSkeletalMeshBulkData::GetIdString() const
{
	FString GuidString = Guid.ToString();
	if (bGuidIsHash)
	{
		GuidString += TEXT("X");
	}
	return GuidString;
}

void FRawSkeletalMeshBulkData::UseHashAsGuid(UObject* Owner)
{
	// Build the hash from the path name + the contents of the bulk data.
	FSHA1 Sha;
	TArray<TCHAR, FString::AllocatorType> OwnerName = Owner->GetPathName().GetCharArray();
	Sha.Update((uint8*)OwnerName.GetData(), OwnerName.Num() * OwnerName.GetTypeSize());
	if (BulkData.GetBulkDataSize() > 0)
	{
		uint8* Buffer = (uint8*)BulkData.Lock(LOCK_READ_ONLY);
		Sha.Update(Buffer, BulkData.GetBulkDataSize());
		BulkData.Unlock();
	}
	Sha.Final();

	// Retrieve the hash and use it to construct a pseudo-GUID. Use bGuidIsHash to distinguish from real guids.
	uint32 Hash[5];
	Sha.GetHash((uint8*)Hash);
	Guid = FGuid(Hash[0] ^ Hash[4], Hash[1], Hash[2], Hash[3]);
	bGuidIsHash = true;
}

FByteBulkData& FRawSkeletalMeshBulkData::GetBulkData()
{
	return BulkData;
}

const FByteBulkData& FRawSkeletalMeshBulkData::GetBulkData() const
{
	return BulkData;
}

/************************************************************************
* FWedgePosition
*/
void FWedgePosition::FindMatchingPositionWegdeIndexes(const FVector3f &Position, float ComparisonThreshold, TArray<int32>& OutResults)
{
	int32 SortedPositionNumber = SortedPositions.Num();
	OutResults.Empty();
	if (SortedPositionNumber == 0)
	{
		//No possible match
		return;
	}
	FWedgePositionHelper::FIndexAndZ PositionIndexAndZ(INDEX_NONE, Position);
	int32 SortedIndex = SortedPositions.Num()/2;
	int32 StartIndex = 0;
	int32 LastTopIndex = SortedPositions.Num();
	int32 LastBottomIndex = 0;
	int32 SearchIterationCount = 0;

	{
		double Increments = ((double)SortedPositions[SortedPositionNumber - 1].Z - (double)SortedPositions[0].Z) / (double)SortedPositionNumber;

		//Optimize the iteration count when a value is not in the middle
		SortedIndex = FMath::RoundToInt(((double)PositionIndexAndZ.Z - (double)SortedPositions[0].Z) / Increments);
	}

	for (SearchIterationCount = 0; SortedPositions.IsValidIndex(SortedIndex); ++SearchIterationCount)
	{
		if (LastTopIndex - LastBottomIndex < 5)
		{
			break;
		}
		if (FMath::Abs(PositionIndexAndZ.Z - SortedPositions[SortedIndex].Z) < ComparisonThreshold)
		{
			//Continue since we want the lowest start
			LastTopIndex = SortedIndex;
			SortedIndex = LastBottomIndex + ((LastTopIndex - LastBottomIndex) / 2);
			if (SortedIndex <= LastBottomIndex)
			{
				break;
			}
		}
		else if (PositionIndexAndZ.Z > SortedPositions[SortedIndex].Z + ComparisonThreshold)
		{
			LastBottomIndex = SortedIndex;
			SortedIndex = SortedIndex + FMath::Max(((LastTopIndex - SortedIndex) / 2), 1);
		}
		else
		{
			LastTopIndex = SortedIndex;
			SortedIndex = SortedIndex - FMath::Max(((SortedIndex - LastBottomIndex) / 2), 1);
		}
	}
	
	//////////////////////////////////////////////////////////////////////////
	//Closest point data (!bExactMatch)
	float MinDistance = UE_MAX_FLT;
	int32 ClosestIndex = LastBottomIndex;

	for (int32 i = LastBottomIndex; i < SortedPositionNumber; i++)
	{
		//Get fast to the close position
		if (PositionIndexAndZ.Z > SortedPositions[i].Z + ComparisonThreshold)
		{
			continue;
		}
		//break when we pass point close to the position
		if (SortedPositions[i].Z > PositionIndexAndZ.Z + ComparisonThreshold)
			break; // can't be any more dups

		//Point is close to the position, verify it
		const FVector3f& PositionA = Points[Wedges[SortedPositions[i].Index].VertexIndex];
		if (FWedgePositionHelper::PointsEqual(PositionA, Position, ComparisonThreshold))
		{
			OutResults.Add(SortedPositions[i].Index);
		}
	}
}

void FOctreeQueryHelper::FindNearestWedgeIndexes(const FVector3f& SearchPosition, TArray<FWedgeInfo>& OutNearestWedges)
{
	if (WedgePosOctree == nullptr)
	{
		return;
	}

	OutNearestWedges.Empty();
	const float OctreeExtent = WedgePosOctree->GetRootBounds().Extent.Size3();
	//Use the max between 1e-4 cm and 1% of the bounding box extend
	FVector Extend(FMath::Max(UE_KINDA_SMALL_NUMBER, OctreeExtent*0.005f));

	//Pass Extent size % of the Octree bounding box extent
	//PassIndex 0 -> 0.5%
	//PassIndex n -> 0.05*n
	//PassIndex 1 -> 5%
	//PassIndex 2 -> 10%
	//...
	for(int32 PassIndex = 0; PassIndex < 5; ++PassIndex)
	{
		// Query the octree to find the vertices close(inside the extend) to the SearchPosition
		WedgePosOctree->FindElementsWithBoundsTest(FBoxCenterAndExtent((FVector)SearchPosition, Extend), [&OutNearestWedges](const FWedgeInfo& WedgeInfo)
		{
			// Add all of the elements in the current node to the list of points to consider for closest point calculations
			OutNearestWedges.Add(WedgeInfo);
		});
		if (OutNearestWedges.Num() == 0)
		{
			float ExtentPercent = 0.05f*((float)PassIndex+1.0f);
			Extend = FVector(FMath::Max(UE_KINDA_SMALL_NUMBER, OctreeExtent * ExtentPercent));
		}
		else
		{
			break;
		}
	}
}

void FWedgePosition::FillWedgePosition(
	FWedgePosition& OutOverlappingPosition,
	const TArray<FVector3f>& Points,
	const TArray<SkeletalMeshImportData::FVertex> Wedges,
	const TArray<FVector3f>& TargetPositions,
	float ComparisonThreshold)
{
	OutOverlappingPosition.Points= Points;
	OutOverlappingPosition.Wedges = Wedges;
	const int32 NumWedges = OutOverlappingPosition.Wedges.Num();
	// Create a list of vertex Z/index pairs
	OutOverlappingPosition.SortedPositions.Reserve(NumWedges);
	for (int32 WedgeIndex = 0; WedgeIndex < NumWedges; WedgeIndex++)
	{
		new(OutOverlappingPosition.SortedPositions)FWedgePositionHelper::FIndexAndZ(WedgeIndex, OutOverlappingPosition.Points[OutOverlappingPosition.Wedges[WedgeIndex].VertexIndex]);
	}

	// Sort the vertices by z value
	OutOverlappingPosition.SortedPositions.Sort(FWedgePositionHelper::FCompareIndexAndZ());


	FBox3f OldBounds(OutOverlappingPosition.Points);
	//Make sure the bounds include the target bounds so we find a match for every target vertex
	OldBounds += FBox3f(TargetPositions);
	OutOverlappingPosition.WedgePosOctree = new TWedgeInfoPosOctree((FVector)OldBounds.GetCenter(), OldBounds.GetExtent().GetMax());

	// Add each old vertex to the octree
	for (int32 WedgeIndex = 0; WedgeIndex < NumWedges; ++WedgeIndex)
	{
		FWedgeInfo WedgeInfo;
		WedgeInfo.WedgeIndex = WedgeIndex;
		WedgeInfo.Position = (FVector)OutOverlappingPosition.Points[OutOverlappingPosition.Wedges[WedgeIndex].VertexIndex];
		OutOverlappingPosition.WedgePosOctree->AddElement(WedgeInfo);
	}
}

namespace InternalImportDataHelper
{
	struct FEdgeMapKey
	{
		int32 VertexIndexes[2];

		FEdgeMapKey(const int32 VertexIndex0, const int32 VertexIndex1)
		{
			const bool bSwap = (VertexIndex0 > VertexIndex1);
			VertexIndexes[0] = (bSwap) ? VertexIndex1 : VertexIndex0;
			VertexIndexes[1] = (bSwap) ? VertexIndex0 : VertexIndex1;
		}

	};

	bool operator==(const FEdgeMapKey& Lhs, const FEdgeMapKey& Rhs)
	{
		if (Lhs.VertexIndexes[0] == Rhs.VertexIndexes[0])
		{
			if (Lhs.VertexIndexes[1] == Rhs.VertexIndexes[1])
			{
				return true;
			}
		}
		else if (Lhs.VertexIndexes[0] == Rhs.VertexIndexes[1])
		{
			if (Lhs.VertexIndexes[1] == Rhs.VertexIndexes[0])
			{
				return true;
			}
		}
		return false;
	}

	bool operator!=(const FEdgeMapKey& Lhs, const FEdgeMapKey& Rhs)
	{
		return !(Lhs == Rhs);
	}

	FORCEINLINE uint32 GetTypeHash(const FEdgeMapKey& EdgeMapKey)
	{
		uint64 Index0Hash = static_cast<uint64>(EdgeMapKey.VertexIndexes[0]);
		uint64 Index1Hash = static_cast<uint64>(EdgeMapKey.VertexIndexes[1]);
		uint64 HashBig64 = Index0Hash + (Index1Hash << 32);
		return FCrc::MemCrc_DEPRECATED(&HashBig64, sizeof(uint64));
	}

	struct FEdgeInfo
	{
		TArray<TTuple<int32, int32>> WedgeIndexes;
		TArray<int32> ConnectedFaces;
	};
} //namespace InternalImportDataHelper

void FSkeletalMeshImportData::ComputeSmoothGroupFromNormals()
{
	//TRACE_CPUPROFILER_EVENT_SCOPE(ComputeSmoothGroupFromNormals);
	const int32 FaceCount = Faces.Num();

	//Create edge connection data
	TMap<InternalImportDataHelper::FEdgeMapKey, InternalImportDataHelper::FEdgeInfo> EdgeInfos;
	EdgeInfos.Reserve(FaceCount * 3);
	for (int32 FaceIndex = 0; FaceIndex < FaceCount; ++FaceIndex)
	{
		SkeletalMeshImportData::FTriangle& Face = Faces[FaceIndex];
		const int32 BaseWedgeIndex = FaceIndex * 3;
		for (int32 Corner = 0; Corner < 3; ++Corner)
		{
			const int32 WedgeIndex = BaseWedgeIndex + Corner;
			const int32 WedgeIndexNext = BaseWedgeIndex + ((Corner + 1) % 3);
			SkeletalMeshImportData::FVertex& Wedge = Wedges[WedgeIndex];
			SkeletalMeshImportData::FVertex& WedgeNext = Wedges[WedgeIndexNext];
			InternalImportDataHelper::FEdgeMapKey EdgeMapKey(Wedge.VertexIndex, WedgeNext.VertexIndex);
			InternalImportDataHelper::FEdgeInfo& EdgeInfo = EdgeInfos.FindOrAdd(EdgeMapKey);
			EdgeInfo.ConnectedFaces.Add(FaceIndex);
			TTuple<int32, int32> TupleWedges = (WedgeIndex <= WedgeIndexNext) ? TTuple<int32, int32>(WedgeIndex, WedgeIndexNext) : TTuple<int32, int32>(WedgeIndexNext, WedgeIndex);
			EdgeInfo.WedgeIndexes.Add(TupleWedges);
		}
	}
	//////////////////////////////////////////////////////////////////////////

	TMap<int32, uint32> FaceSmoothGroup;
	FaceSmoothGroup.Reserve(FaceCount);
	TArray<bool> ConsumedFaces;
	ConsumedFaces.AddZeroed(FaceCount);

	TMap < int32, uint32> FaceAvoidances;

	TArray<int32> SoftEdgeNeigbors;
	TArray<int32> ConnectedFaces;
	TArray<int32> LastConnectedFaces;

	for (int32 FaceIndex = 0; FaceIndex < FaceCount; ++FaceIndex)
	{
		if (ConsumedFaces[FaceIndex])
		{
			continue;
		}

		ConnectedFaces.Reset();
		LastConnectedFaces.Reset();
		ConnectedFaces.Add(FaceIndex);
		LastConnectedFaces.Add(INDEX_NONE);
		while (ConnectedFaces.Num() > 0)
		{
			check(LastConnectedFaces.Num() == ConnectedFaces.Num());
			int32 LastFaceIndex = LastConnectedFaces.Pop(EAllowShrinking::No);
			int32 CurrentFaceIndex = ConnectedFaces.Pop(EAllowShrinking::No);
			if (ConsumedFaces[CurrentFaceIndex])
			{
				continue;
			}
			SoftEdgeNeigbors.Reset();
			uint32& SmoothGroup = FaceSmoothGroup.FindOrAdd(CurrentFaceIndex);
			uint32 AvoidSmoothGroup = 0;
			uint32 NeighborSmoothGroup = 0;
			const uint32 LastSmoothGroupValue = (LastFaceIndex == INDEX_NONE) ? 0 : FaceSmoothGroup[LastFaceIndex];

			SkeletalMeshImportData::FTriangle& Face = Faces[CurrentFaceIndex];
			const int32 BaseWedgeIndex = CurrentFaceIndex * 3;
			for (int32 Corner = 0; Corner < 3; ++Corner)
			{
				const int32 CornerNext = (Corner + 1) % 3;
				const int32 WedgeIndex = BaseWedgeIndex + Corner;
				const int32 WedgeIndexNext = BaseWedgeIndex + CornerNext;
				SkeletalMeshImportData::FVertex& Wedge = Wedges[WedgeIndex];
				SkeletalMeshImportData::FVertex& WedgeNext = Wedges[WedgeIndexNext];
				const FVector3f& WedgePosition = Points[Wedge.VertexIndex];
				const FVector3f& WedgePositionNext = Points[WedgeNext.VertexIndex];
				InternalImportDataHelper::FEdgeMapKey EdgeMapKey(Wedge.VertexIndex, WedgeNext.VertexIndex);
				InternalImportDataHelper::FEdgeInfo& EdgeInfo = EdgeInfos.FindChecked(EdgeMapKey);
				for (int32 EdgeInfoConnectedFaceIndex = 0; EdgeInfoConnectedFaceIndex < EdgeInfo.ConnectedFaces.Num(); ++EdgeInfoConnectedFaceIndex)
				{
					const int32 ConnectedFaceIndex = EdgeInfo.ConnectedFaces[EdgeInfoConnectedFaceIndex];
					if (ConnectedFaceIndex == CurrentFaceIndex)
					{
						continue;
					}
					const SkeletalMeshImportData::FTriangle& ConnectedFace = Faces[ConnectedFaceIndex];
					const TTuple<int32, int32> ConnectedWedgeIndexes = EdgeInfo.WedgeIndexes[EdgeInfoConnectedFaceIndex];
					//Is the edge is hard or smooth? Test only the normals, since we create the smooth group from the normals
					SkeletalMeshImportData::FVertex& ConnectedWedge = Wedges[ConnectedWedgeIndexes.Key];
					SkeletalMeshImportData::FVertex& ConnectedWedgeNext = Wedges[ConnectedWedgeIndexes.Value];
					uint32 SmoothValue = 0;
					if (FaceSmoothGroup.Contains(ConnectedFaceIndex))
					{
						SmoothValue = FaceSmoothGroup[ConnectedFaceIndex];
					}
					bool bSmoothEdgeNormal = false;
					const int32 ConnectedCorner = ((ConnectedWedge.VertexIndex == Wedge.VertexIndex) ? ConnectedWedgeIndexes.Key : ConnectedWedgeIndexes.Value) % 3;
					const int32 ConnectedCornerNext = ((ConnectedWedge.VertexIndex == Wedge.VertexIndex) ? ConnectedWedgeIndexes.Value : ConnectedWedgeIndexes.Key) % 3;
					if (ConnectedFace.TangentZ[ConnectedCorner].Equals(Face.TangentZ[Corner]))
					{
						if (ConnectedFace.TangentZ[ConnectedCornerNext].Equals(Face.TangentZ[CornerNext]))
						{
							bSmoothEdgeNormal = true;
						}
					}

					if (!bSmoothEdgeNormal) //Hard Edge
					{
						AvoidSmoothGroup |= SmoothValue;
					}
					else
					{
						NeighborSmoothGroup |= SmoothValue;
						//Put all none hard edge polygon in the next iteration
						if (!ConsumedFaces[ConnectedFaceIndex])
						{
							ConnectedFaces.Add(ConnectedFaceIndex);
							LastConnectedFaces.Add(CurrentFaceIndex);
						}
						else
						{
							SoftEdgeNeigbors.Add(ConnectedFaceIndex);
						}
					}
				}
			}

			if (AvoidSmoothGroup != 0)
			{
				FaceAvoidances.FindOrAdd(CurrentFaceIndex) = AvoidSmoothGroup;
				//find neighbor avoidance
				for (int32 NeighborID : SoftEdgeNeigbors)
				{
					if (!FaceAvoidances.Contains(NeighborID))
					{
						continue;
					}
					AvoidSmoothGroup |= FaceAvoidances[NeighborID];
				}
				uint32 NewSmoothGroup = 1;
				while ((NewSmoothGroup & AvoidSmoothGroup) != 0 && NewSmoothGroup < MAX_uint32)
				{
					//Shift the smooth group
					NewSmoothGroup = NewSmoothGroup << 1;
				}
				SmoothGroup = NewSmoothGroup;
				//Apply to all neighboard
				for (int32 NeighborID : SoftEdgeNeigbors)
				{
					FaceSmoothGroup[NeighborID] |= NewSmoothGroup;
				}
			}
			else if (NeighborSmoothGroup != 0)
			{
				SmoothGroup |= LastSmoothGroupValue | NeighborSmoothGroup;
			}
			else
			{
				SmoothGroup = 1;
			}
			ConsumedFaces[CurrentFaceIndex] = true;
		}
	}
	//Set the resulting smooth group for all the faces
	check(FaceSmoothGroup.Num() == FaceCount);
	for (int32 FaceIndex = 0; FaceIndex < FaceCount; ++FaceIndex)
	{
		Faces[FaceIndex].SmoothingGroups = FaceSmoothGroup[FaceIndex];
	}
}

void FSkeletalMeshImportData::AddMorphTarget(
	FName InMorphTargetName,
	const FMorphTargetLODModel& InMorphTargetModel,
	const TArray<uint32>& InVertexMap
	)
{
	MorphTargetNames.Add(InMorphTargetName.ToString());

	FSkeletalMeshImportData& MorphTargetDst = MorphTargets.AddDefaulted_GetRef();
	TSet<uint32>& ModifiedPoints = MorphTargetModifiedPoints.AddDefaulted_GetRef();

	for (const FMorphTargetDelta& Delta: InMorphTargetModel.Vertices)
	{
		if (InVertexMap.IsValidIndex(Delta.SourceIdx))
		{
			const int32 MappedIndex = InVertexMap[Delta.SourceIdx];
			if (Points.IsValidIndex(MappedIndex))
			{
				ModifiedPoints.Add(MappedIndex);
				MorphTargetDst.Points.Add(Delta.PositionDelta + Points[MappedIndex]);
			}
		}
	}
}

void FSkeletalMeshImportData::AddSkinWeightProfile(
	FName InProfileName,
	const FImportedSkinWeightProfileData& InProfileData,
	const TArray<int32>& InVertexMap,
	const TArray<FBoneIndexType>& InBoneIndexMap
	)
{
	AlternateInfluenceProfileNames.Add(InProfileName.ToString());
	FSkeletalMeshImportData& TargetProfileData = AlternateInfluences.AddDefaulted_GetRef();

	TargetProfileData.Points = Points;
	TargetProfileData.Wedges = Wedges;
	TargetProfileData.Faces = Faces;
	TargetProfileData.RefBonesBinary = RefBonesBinary;
	TargetProfileData.Influences.Reserve(InProfileData.SourceModelInfluences.Num());

	// We can't use the SourceModelInfluences since it may come from a alt skin mesh that didn't have the exact
	// same point location and was therefore projected onto base base mesh. However, those points no longer exist
	// and so we can't rely on them. Instead we reverse map back the LOD model influences onto the import model
	// using the vertex map. We need to keep track of which target point we've set since points may have gotten split,
	// and the vertex map can point to the same point multiple times, and we don't want to end up with double weights.
	TSet<int32> ProcessedVertex;
	for (int32 VertexIndex = 0; VertexIndex < InProfileData.SkinWeights.Num(); VertexIndex++)
	{
		const FRawSkinWeight& SourceWeights = InProfileData.SkinWeights[VertexIndex];
		const int32 TargetVertexIndex = InVertexMap[VertexIndex];
		if (ProcessedVertex.Contains(TargetVertexIndex))
		{
			continue;
		}

		ProcessedVertex.Add(TargetVertexIndex);
		
		for (int32 InfluenceIndex = 0; InfluenceIndex < MAX_TOTAL_INFLUENCES; InfluenceIndex++)
		{
			if (SourceWeights.InfluenceWeights[InfluenceIndex] == 0)
			{
				continue;
			}

			if (InBoneIndexMap.IsValidIndex(SourceWeights.InfluenceBones[InfluenceIndex]))
			{
				SkeletalMeshImportData::FRawBoneInfluence TargetInfluence;
				TargetInfluence.VertexIndex = TargetVertexIndex;
				TargetInfluence.BoneIndex = InBoneIndexMap[SourceWeights.InfluenceBones[InfluenceIndex]];
				TargetInfluence.Weight = SourceWeights.InfluenceWeights[InfluenceIndex] / 65535.0f;
				TargetProfileData.Influences.Add(TargetInfluence);
			}
		}
	}
}



void FSkeletalMeshImportData::CleanUpUnusedMaterials()
{
	if (Materials.Num() <= 0)
	{
		return;
	}

	TArray<SkeletalMeshImportData::FMaterial> ExistingMatList = Materials;

	TArray<uint8> UsedMaterialIndex;
	// Find all material that are use by the mesh faces
	int32 FaceNum = Faces.Num();
	for (int32 TriangleIndex = 0; TriangleIndex < FaceNum; TriangleIndex++)
	{
		SkeletalMeshImportData::FTriangle& Triangle = Faces[TriangleIndex];
		UsedMaterialIndex.AddUnique(Triangle.MatIndex);
	}
	//Remove any unused material.
	if (UsedMaterialIndex.Num() < ExistingMatList.Num())
	{
		TArray<int32> RemapIndex;
		TArray< SkeletalMeshImportData::FMaterial >& NewMatList = Materials;
		NewMatList.Empty();
		for (int32 ExistingMatIndex = 0; ExistingMatIndex < ExistingMatList.Num(); ++ExistingMatIndex)
		{
			if (UsedMaterialIndex.Contains((uint8)ExistingMatIndex))
			{
				RemapIndex.Add(NewMatList.Add(ExistingMatList[ExistingMatIndex]));
			}
			else
			{
				RemapIndex.Add(INDEX_NONE);
			}
		}
		MaxMaterialIndex = 0;
		//Remap the face material index
		for (int32 TriangleIndex = 0; TriangleIndex < FaceNum; TriangleIndex++)
		{
			SkeletalMeshImportData::FTriangle& Triangle = Faces[TriangleIndex];
			check(RemapIndex[Triangle.MatIndex] != INDEX_NONE);
			Triangle.MatIndex = RemapIndex[Triangle.MatIndex];
			MaxMaterialIndex = FMath::Max<uint32>(MaxMaterialIndex, Triangle.MatIndex);
		}
	}
}

namespace SmoothGroupHelper
{
	struct tFaceRecord
	{
		int32 FaceIndex;
		int32 HoekIndex;
		int32 WedgeIndex;
		uint32 SmoothFlags;
		uint32 FanFlags;
	};

	struct VertsFans
	{
		TArray<tFaceRecord> FaceRecord;
		int32 FanGroupCount;
	};

	struct tInfluences
	{
		TArray<int32> RawInfIndices;
	};

	struct tWedgeList
	{
		TArray<int32> WedgeList;
	};

	struct tFaceSet
	{
		TArray<int32> Faces;
	};

	// Check whether faces have at least two vertices in common. These must be POINTS - don't care about wedges.
	bool FacesAreSmoothlyConnected(FSkeletalMeshImportData& ImportData, int32 Face1, int32 Face2)
	{

		//if( ( Face1 >= Thing->SkinData.Faces.Num()) || ( Face2 >= Thing->SkinData.Faces.Num()) ) return false;

		if (Face1 == Face2)
		{
			return true;
		}

		// Smoothing groups match at least one bit in binary AND ?
		if ((ImportData.Faces[Face1].SmoothingGroups & ImportData.Faces[Face2].SmoothingGroups) == 0)
		{
			return false;
		}

		int32 VertMatches = 0;
		for (int32 i = 0; i < 3; i++)
		{
			int32 Point1 = ImportData.Wedges[ImportData.Faces[Face1].WedgeIndex[i]].VertexIndex;

			for (int32 j = 0; j < 3; j++)
			{
				int32 Point2 = ImportData.Wedges[ImportData.Faces[Face2].WedgeIndex[j]].VertexIndex;
				if (Point2 == Point1)
				{
					VertMatches++;
				}
			}
		}

		return (VertMatches >= 2);
	}
} //namespace SmoothGroupHelper

void FSkeletalMeshImportData::SplitVerticesBySmoothingGroups()
{
	//
	// Connectivity: triangles with non-matching smoothing groups will be physically split.
	//
	// -> Splitting involves: the UV+material-containing vertex AND the 3d point.
	//
	// -> Tally smoothing groups for each and every (textured) vertex.
	//
	// -> Collapse: 
	// -> start from a vertex and all its adjacent triangles - go over
	// each triangle - if any connecting one (sharing more than one vertex) gives a smoothing match,
	// accumulate it. Then IF more than one resulting section, 
	// ensure each boundary 'vert' is split _if not already_ to give each smoothing group
	// independence from all others.
	//

	int32 TotalSmoothMatches = 0;
	int32 TotalConnexChex = 0;

	// Link _all_ faces to vertices.	
	TArray<SmoothGroupHelper::VertsFans>  Fans;
	TArray<SmoothGroupHelper::tInfluences> PointInfluences;
	TArray<SmoothGroupHelper::tWedgeList>  PointWedges;

	Fans.AddZeroed(Points.Num());//Fans.AddExactZeroed(			Thing->SkinData.Points.Num() );
	PointInfluences.AddZeroed(Points.Num());//PointInfluences.AddExactZeroed( Thing->SkinData.Points.Num() );
	PointWedges.AddZeroed(Points.Num());//PointWedges.AddExactZeroed(	 Thing->SkinData.Points.Num() );

	// Existing points map 1:1
	PointToRawMap.AddUninitialized(Points.Num());
	for (int32 i = 0; i < Points.Num(); i++)
	{
		PointToRawMap[i] = i;
	}

	for (int32 i = 0; i < Influences.Num(); i++)
	{
		if (PointInfluences.Num() <= Influences[i].VertexIndex)
		{
			PointInfluences.AddZeroed(Influences[i].VertexIndex - PointInfluences.Num() + 1);
		}
		PointInfluences[Influences[i].VertexIndex].RawInfIndices.Add(i);
	}

	for (int32 i = 0; i < Wedges.Num(); i++)
	{
		if (uint32(PointWedges.Num()) <= Wedges[i].VertexIndex)
		{
			PointWedges.AddZeroed(Wedges[i].VertexIndex - PointWedges.Num() + 1);
		}

		PointWedges[Wedges[i].VertexIndex].WedgeList.Add(i);
	}

	for (int32 f = 0; f < Faces.Num(); f++)
	{
		// For each face, add a pointer to that face into the Fans[vertex].
		for (int32 i = 0; i < 3; i++)
		{
			int32 WedgeIndex = Faces[f].WedgeIndex[i];
			int32 PointIndex = Wedges[WedgeIndex].VertexIndex;
			SmoothGroupHelper::tFaceRecord NewFR;

			NewFR.FaceIndex = f;
			NewFR.HoekIndex = i;
			NewFR.WedgeIndex = WedgeIndex; // This face touches the point courtesy of Wedges[Wedgeindex].
			NewFR.SmoothFlags = Faces[f].SmoothingGroups;
			NewFR.FanFlags = 0;
			Fans[PointIndex].FaceRecord.Add(NewFR);
			Fans[PointIndex].FanGroupCount = 0;
		}
	}

	// Investigate connectivity and assign common group numbers (1..+) to the fans' individual FanFlags.
	for (int32 p = 0; p < Fans.Num(); p++) // The fan of faces for each 3d point 'p'.
	{
		// All faces connecting.
		if (Fans[p].FaceRecord.Num() > 0)
		{
			int32 FacesProcessed = 0;
			TArray<SmoothGroupHelper::tFaceSet> FaceSets; // Sets with indices INTO FANS, not into face array.			

			// Digest all faces connected to this vertex (p) into one or more smooth sets. only need to check 
			// all faces MINUS one..
			while (FacesProcessed < Fans[p].FaceRecord.Num())
			{
				// One loop per group. For the current ThisFaceIndex, tally all truly connected ones
				// and put them in a new TArray. Once no more can be connected, stop.

				int32 NewSetIndex = FaceSets.Num(); // 0 to start
				FaceSets.AddZeroed(1);						// first one will be just ThisFaceIndex.

				// Find the first non-processed face. There will be at least one.
				int32 ThisFaceFanIndex = 0;
				{
					int32 SearchIndex = 0;
					while (Fans[p].FaceRecord[SearchIndex].FanFlags == -1) // -1 indicates already  processed. 
					{
						SearchIndex++;
					}
					ThisFaceFanIndex = SearchIndex; //Fans[p].FaceRecord[SearchIndex].FaceIndex; 
				}

				// Initial face.
				FaceSets[NewSetIndex].Faces.Add(ThisFaceFanIndex);   // Add the unprocessed Face index to the "local smoothing group" [NewSetIndex].
				Fans[p].FaceRecord[ThisFaceFanIndex].FanFlags = -1;			  // Mark as processed.
				FacesProcessed++;

				// Find all faces connected to this face, and if there's any
				// smoothing group matches, put it in current face set and mark it as processed;
				// until no more match. 
				int32 NewMatches = 0;
				do
				{
					NewMatches = 0;
					// Go over all current faces in this faceset and set if the FaceRecord (local smoothing groups) has any matches.
					// there will be at least one face already in this faceset - the first face in the fan.
					for (int32 n = 0; n < FaceSets[NewSetIndex].Faces.Num(); n++)
					{
						int32 HookFaceIdx = Fans[p].FaceRecord[FaceSets[NewSetIndex].Faces[n]].FaceIndex;

						//Go over the fan looking for matches.
						for (int32 s = 0; s < Fans[p].FaceRecord.Num(); s++)
						{
							// Skip if same face, skip if face already processed.
							if ((HookFaceIdx != Fans[p].FaceRecord[s].FaceIndex) && (Fans[p].FaceRecord[s].FanFlags != -1))
							{
								TotalConnexChex++;
								// Process if connected with more than one vertex, AND smooth..
								if (SmoothGroupHelper::FacesAreSmoothlyConnected(*this, HookFaceIdx, Fans[p].FaceRecord[s].FaceIndex))
								{
									TotalSmoothMatches++;
									Fans[p].FaceRecord[s].FanFlags = -1; // Mark as processed.
									FacesProcessed++;
									// Add 
									FaceSets[NewSetIndex].Faces.Add(s); // Store FAN index of this face index into smoothing group's faces. 
									// Tally
									NewMatches++;
								}
							} // not the same...
						}// all faces in fan
					} // all faces in FaceSet
				} while (NewMatches);

			}// Repeat until all faces processed.

			// For the new non-initialized  face sets, 
			// Create a new point, influences, and uv-vertex(-ices) for all individual FanFlag groups with an index of 2+ and also remap
			// the face's vertex into those new ones.
			if (FaceSets.Num() > 1)
			{
				for (int32 f = 1; f < FaceSets.Num(); f++)
				{
					check(Points.Num() == PointToRawMap.Num());

					// We duplicate the current vertex. (3d point)
					const int32 NewPointIndex = Points.Num();
					Points.AddUninitialized();
					Points[NewPointIndex] = Points[p];

					PointToRawMap.AddUninitialized();
					PointToRawMap[NewPointIndex] = p;

					// Duplicate all related weights.
					for (int32 t = 0; t < PointInfluences[p].RawInfIndices.Num(); t++)
					{
						// Add new weight
						int32 NewWeightIndex = Influences.Num();
						Influences.AddUninitialized();
						Influences[NewWeightIndex] = Influences[PointInfluences[p].RawInfIndices[t]];
						Influences[NewWeightIndex].VertexIndex = NewPointIndex;
					}

					// Duplicate any and all Wedges associated with it; and all Faces' wedges involved.					
					for (int32 w = 0; w < PointWedges[p].WedgeList.Num(); w++)
					{
						const int32 OldWedgeIndex = PointWedges[p].WedgeList[w];
						Wedges[OldWedgeIndex].VertexIndex = NewPointIndex;
					}
				}
			} //  if FaceSets.Num(). -> duplicate stuff
		}//	while( FacesProcessed < Fans[p].FaceRecord.Num() )
	} // Fans for each 3d point

	if (!ensure(Points.Num() == PointToRawMap.Num()))
	{
		//TODO log a warning to the user

		//Create a valid PointtoRawMap but with bad content
		int32 PointNum = Points.Num();
		PointToRawMap.Empty(PointNum);
		for (int32 PointIndex = 0; PointIndex < PointNum; ++PointIndex)
		{
			PointToRawMap.Add(PointIndex);
		}
	}
}

static void CopySkinWeightsToAttribute(
	const USkeletalMesh *InSkeletalMesh,
	const FName InProfileName,
	const TArray<SkeletalMeshImportData::FRawBoneInfluence>& InInfluences,
	const TArray<FVertexID>& InVertexIDMap,
	const TMap<int32, int32>* InBoneIndexMap,
	FSkinWeightsVertexAttributesRef OutSkinWeightsAttribute
	)
{
	using namespace UE::AnimationCore;
	
	// The weights are stored with links back to the vertices, rather than being compact.
	// Make a copy of the weights, sort them by vertex id and go by equal vertex-id strides.
	// We could do an indirection but the traversal + setup cost is probably not worth it.
	TArray<SkeletalMeshImportData::FRawBoneInfluence> SortedInfluences(InInfluences);
	SortedInfluences.Sort([](const SkeletalMeshImportData::FRawBoneInfluence &A, const SkeletalMeshImportData::FRawBoneInfluence &B)
	{
		return A.VertexIndex < B.VertexIndex;
	});

	// Do the base skin weights first. We do the alternate skin weights later, since they may require geometric remapping. 
	TArray<FBoneWeight> BoneWeights;
	for(int32 StartStride = 0, EndStride = 0; EndStride != SortedInfluences.Num(); StartStride = EndStride)
	{
		const int32 VertexIndex = SortedInfluences[StartStride].VertexIndex;

		// There exist meshes where the influence map got auto-filled with 100% weight on root in by using the wedge count of the raw mesh,
		// due to missing weights (e.g. static mesh imported as a skeletal mesh), and so may refer to vertices that don't exist.
		// We just stop when we get to the broken set and ignore the rest.
		if (VertexIndex >= InVertexIDMap.Num())
		{
			if (InSkeletalMesh)
			{
				FString ProfileDetail;
				if (!InProfileName.IsNone())
				{
					ProfileDetail = FString::Printf(TEXT(" on profile '%s'"), *InProfileName.ToString());
				}
				UE_ASSET_LOG(LogSkeletalMeshLODImporterData, Display, InSkeletalMesh, TEXT("Influences%s refer to non-existent vertices. Please re-import to fix."),
					*ProfileDetail);
			}
			break;
		}
		
		EndStride = StartStride + 1;
		while (EndStride < SortedInfluences.Num() && VertexIndex == SortedInfluences[EndStride].VertexIndex)
		{
			EndStride++;
		}

		BoneWeights.Reset(0);
		for (int32 Idx = StartStride; Idx < EndStride; Idx++)
		{
			const SkeletalMeshImportData::FRawBoneInfluence &RawInfluence = SortedInfluences[Idx];
			int32 BoneIndex = RawInfluence.BoneIndex;
			if (InBoneIndexMap)
			{
				// If we can't map from a source bone to a target bone, we skip the weight. 
				if (const int32* BoneIndexPtr = InBoneIndexMap->Find(BoneIndex))
				{
					BoneIndex = *BoneIndexPtr;
				}
				else
				{
					continue;
				}
			}
			
			FBoneWeight BoneWeight(static_cast<FBoneIndexType>(BoneIndex), RawInfluence.Weight);
			BoneWeights.Add(BoneWeight);
		}

		if (BoneWeights.IsEmpty())
		{
			const FBoneWeight RootBoneWeight(0, 1.0f);
			BoneWeights.Add(RootBoneWeight);
		}

		OutSkinWeightsAttribute.Set(InVertexIDMap[VertexIndex], BoneWeights);		
	}
}

bool FSkeletalMeshImportData::GetMeshDescription(const USkeletalMesh* InSkeletalMesh, const FSkeletalMeshBuildSettings* InBuildSettings, FMeshDescription& OutMeshDescription) const
{
	using namespace UE::AnimationCore;
	
	OutMeshDescription.Empty();
	
	FSkeletalMeshAttributes MeshAttributes(OutMeshDescription);

	MeshAttributes.Register();

	if (Points.IsEmpty() || Faces.IsEmpty())
	{
		return false;
	}

	MeshAttributes.RegisterImportPointIndexAttribute(); 
	
	if (!MeshInfos.IsEmpty())
	{
		MeshAttributes.RegisterSourceGeometryPartsAttributes();
	}
	
	TVertexAttributesRef<FVector3f> VertexPositions = MeshAttributes.GetVertexPositions();
	FSkinWeightsVertexAttributesRef VertexSkinWeights = MeshAttributes.GetVertexSkinWeights();
	
	TVertexAttributesRef<int32> ImportPointIndex;
	if (!PointToRawMap.IsEmpty())
	{
		ImportPointIndex = OutMeshDescription.VertexAttributes().GetAttributesRef<int32>(MeshAttribute::Vertex::ImportPointIndex); 
	}
	
	TVertexInstanceAttributesRef<FVector3f> VertexInstanceNormals = MeshAttributes.GetVertexInstanceNormals();
	TVertexInstanceAttributesRef<FVector3f> VertexInstanceTangents = MeshAttributes.GetVertexInstanceTangents();
	TVertexInstanceAttributesRef<float> VertexInstanceBinormalSigns = MeshAttributes.GetVertexInstanceBinormalSigns();
	TVertexInstanceAttributesRef<FVector4f> VertexInstanceColors = MeshAttributes.GetVertexInstanceColors();
	TVertexInstanceAttributesRef<FVector2f> VertexInstanceUVs = MeshAttributes.GetVertexInstanceUVs();

	TPolygonGroupAttributesRef<FName> PolygonGroupMaterialSlotNames = MeshAttributes.GetPolygonGroupMaterialSlotNames();

	FSkeletalMeshAttributes::FBoneNameAttributesRef BoneNames = MeshAttributes.GetBoneNames();
	FSkeletalMeshAttributes::FBoneParentIndexAttributesRef BoneParentIndices = MeshAttributes.GetBoneParentIndices();
	FSkeletalMeshAttributes::FBonePoseAttributesRef BonePoses = MeshAttributes.GetBonePoses();

	// Register morph targets, rename as needed, in case there are invalid names provided (e.g. empty strings, duplicates)
	TMap<int32, FString> ValidMorphTargets;
	if (ensure(MorphTargetNames.Num() == MorphTargets.Num()) &&
		ensure(MorphTargets.Num() == MorphTargetModifiedPoints.Num()))
	{
		TSet<FString> CheckDuplicateSet;
		
		for (int32 MorphTargetIndex = 0; MorphTargetIndex < MorphTargets.Num(); MorphTargetIndex++)
		{
			const FString MorphTargetName(MorphTargetNames[MorphTargetIndex]);
			FString MorphTargetNameUsed(MorphTargetName);
			const TArray<FVector3f>& MorphPoints = MorphTargets[MorphTargetIndex].Points;
			const TSet<uint32>& ModifiedPoints = MorphTargetModifiedPoints[MorphTargetIndex];
			
			if (MorphPoints.Num() != ModifiedPoints.Num())
			{
				if (InSkeletalMesh)
				{
					UE_ASSET_LOG(LogSkeletalMeshLODImporterData, Warning, InSkeletalMesh, TEXT("Morph target '%s' has mismatched compressed and modified points. Morph target will be dropped. Re-import to fix."),
						*MorphTargetName);  
				}
				continue;
			}

			// Ensure modified points indices are valid
			if (Algo::AnyOf(ModifiedPoints, [this](uint32 PointIndex) { return !Points.IsValidIndex(PointIndex); }))
			{
				UE_ASSET_LOG(LogSkeletalMeshLODImporterData, Warning, InSkeletalMesh, TEXT("Morph target '%s' has modified points that don't exist on the base mesh. Morph target will be dropped. Re-import to fix."),
					*MorphTargetName);  
				continue;
			}

			if (MorphTargetNameUsed.IsEmpty())
			{
				MorphTargetNameUsed = TEXT("Unnamed");
				UE_ASSET_LOG(LogSkeletalMeshLODImporterData, Display, InSkeletalMesh, TEXT("Morph target found with no name. Renamed to 'Unnamed'."));  
			}

			// Has this name been used already?
			FString MorphTargetNameBase(MorphTargetNameUsed);
			for(int32 Suffix = 1; CheckDuplicateSet.Contains(MorphTargetNameUsed); Suffix++)
			{
				MorphTargetNameUsed = FString::Printf(TEXT("%s_%d"), *MorphTargetNameBase, Suffix);
			}
			if (MorphTargetNameUsed != MorphTargetNameBase)
			{
				UE_ASSET_LOG(LogSkeletalMeshLODImporterData, Warning, InSkeletalMesh, TEXT("Duplicate morph target '%s' found, renamed to '%s'."),
					*MorphTargetNameBase, *MorphTargetNameUsed);  
			}

			// Register a morph attribute but not with normals since the import data doesn't
			// have any.
			if (ensure(MeshAttributes.RegisterMorphTargetAttribute(*MorphTargetNameUsed, false)))
			{
				ValidMorphTargets.Add(MorphTargetIndex, MorphTargetNameUsed);
				CheckDuplicateSet.Add(MorphTargetNameUsed);
			}
		}
	}

	// Register alternate skin weight profiles.
	TSet<FString> ValidSkinWeights;
	if (ensure(AlternateInfluenceProfileNames.Num() == AlternateInfluences.Num()))
	{
		for (int32 AlternateInfluenceIndex = 0; AlternateInfluenceIndex < AlternateInfluences.Num(); AlternateInfluenceIndex++)
		{
			const FString AlternateInfluenceProfileName(AlternateInfluenceProfileNames[AlternateInfluenceIndex]);
			const FSkeletalMeshImportData& AlternateInfluenceMesh = AlternateInfluences[AlternateInfluenceIndex];
			
			// We minimally need the same point count.
			// Q: Should we match on topology as well?
			if (AlternateInfluenceMesh.Points.Num() != Points.Num())
			{
				if (InSkeletalMesh)
				{
					UE_ASSET_LOG(LogSkeletalMeshLODImporterData, Display, InSkeletalMesh, TEXT("Alternate influence profile '%s' point count is different from base mesh. Profile will be dropped since it cannot be matched to the base mesh."),
						*AlternateInfluenceProfileName);  
				}
				continue;
			}

			// The set of bound bones must completely include the bones in the base mesh. Otherwise, we cannot map the influences over to
			// the base skeleton.
			TSet<FString> BaseBones;
			TSet<int32> MissingBoneIndices;
			for (int32 BoneIndex = 0; BoneIndex < RefBonesBinary.Num(); BoneIndex++)
			{
				BaseBones.Add(RefBonesBinary[BoneIndex].Name);
			}
			for (int32 BoneIndex = 0; BoneIndex < AlternateInfluenceMesh.RefBonesBinary.Num(); BoneIndex++)
			{
				if (!BaseBones.Contains(AlternateInfluenceMesh.RefBonesBinary[BoneIndex].Name))
				{
					MissingBoneIndices.Add(BoneIndex);
				}
			}

			TSet<int32> InfluencesBoundToMissingBones; 
			for (const SkeletalMeshImportData::FRawBoneInfluence& Influence: AlternateInfluenceMesh.Influences)
			{
				if (MissingBoneIndices.Contains(Influence.BoneIndex))
				{
					InfluencesBoundToMissingBones.Add(Influence.BoneIndex);
				}
			}

			if (!InfluencesBoundToMissingBones.IsEmpty())
			{
				TArray<FString> MissingBoneNames;
				for (const int32 BoneIndex: InfluencesBoundToMissingBones)
				{
					MissingBoneNames.Add(FString::Printf(TEXT("'%s'"), *AlternateInfluenceMesh.RefBonesBinary[BoneIndex].Name));
				}
				MissingBoneNames.Sort();
				
				if (InSkeletalMesh)
				{
					UE_ASSET_LOG(LogSkeletalMeshLODImporterData, Display, InSkeletalMesh, TEXT("Alternate influence profile '%s' binds to one or more bones (%s) that do not exist on base mesh's skeleton. Those bone bindings will be dropped, which may reduce visual quality."),
						*AlternateInfluenceProfileName, *FString::Join(MissingBoneNames, TEXT(", ")));  
				}
			}

			if (ensure(MeshAttributes.RegisterSkinWeightAttribute(*AlternateInfluenceProfileName)))
			{
				ValidSkinWeights.Add(AlternateInfluenceProfileNames[AlternateInfluenceIndex]);
			}

		}
	}

	// Register vertex attributes.
	TSet<FString> ValidAttributes;
	for (int32 AttributeIndex = 0; AttributeIndex < VertexAttributes.Num(); AttributeIndex++)
	{
		const FString& VertexAttributeName = VertexAttributeNames[AttributeIndex];
		const SkeletalMeshImportData::FVertexAttribute& VertexAttribute = VertexAttributes[AttributeIndex];
		if (VertexAttribute.AttributeValues.Num() != (Points.Num() * VertexAttribute.ComponentCount))
		{
			UE_ASSET_LOG(LogSkeletalMeshLODImporterData, Warning, InSkeletalMesh, TEXT("Vertex attribute '%s' value count (%d) does not match the mesh's point count (%d)."),
				*VertexAttributeName, VertexAttribute.AttributeValues.Num() / VertexAttribute.ComponentCount, Points.Num());
			continue;
		}

		EMeshAttributeFlags DefaultAttributeFlags = EMeshAttributeFlags::Mergeable | EMeshAttributeFlags::Lerpable;
		
		FName RegisteredName(VertexAttributeName);

		// Ignore attributes with reserved names. This should have been handled at import time or when the attribute
		// was created/renamed.
		if (!ensure(!FSkeletalMeshAttributes::IsReservedAttributeName(RegisteredName)))
		{
			continue;
		}

		bool bIsValidAttribute = false;
		switch(VertexAttribute.ComponentCount)
		{
		case 1:
			bIsValidAttribute = OutMeshDescription.VertexAttributes().RegisterAttribute<float>(RegisteredName, 1, 0.0f, DefaultAttributeFlags).IsValid();
			break;
		case 2:
			bIsValidAttribute = OutMeshDescription.VertexAttributes().RegisterAttribute<FVector2f>(RegisteredName, 1, FVector2f::Zero(), DefaultAttributeFlags).IsValid();
			break;
		case 3:
			bIsValidAttribute = OutMeshDescription.VertexAttributes().RegisterAttribute<FVector3f>(RegisteredName, 1, FVector3f::Zero(), DefaultAttributeFlags).IsValid();
			break;
		case 4:
			bIsValidAttribute = OutMeshDescription.VertexAttributes().RegisterAttribute<FVector4f>(RegisteredName, 1, FVector4f::Zero(), DefaultAttributeFlags).IsValid();
			break;
		default:
			continue;
		}

		if (ensure(bIsValidAttribute))
		{
			ValidAttributes.Add(VertexAttributeName);
		}
	}
	
	VertexInstanceUVs.SetNumChannels(NumTexCoords);
	OutMeshDescription.SetNumUVChannels(NumTexCoords);
	
	// Avoid repeated allocations and reserve the target buffers right off the bat.
	OutMeshDescription.ReserveNewPolygonGroups(Materials.Num());
	OutMeshDescription.ReserveNewPolygons(Faces.Num());
	OutMeshDescription.ReserveNewTriangles(Faces.Num());
	OutMeshDescription.ReserveNewVertexInstances(Wedges.Num());
	OutMeshDescription.ReserveNewVertices(Points.Num());
	MeshAttributes.ReserveNewBones(RefBonesBinary.Num());
	

	// Copy the vertex positions first and maintain a map so that we can go from the import data's raw vertex index
	// to the mesh description's VertexID.
	TArray<FVertexID> VertexIDMap;
	VertexIDMap.Reserve(Points.Num());
	for (int32 Idx = 0; Idx < Points.Num(); Idx++)
	{
		const FVertexID VertexID = OutMeshDescription.CreateVertex();
		VertexIDMap.Add(VertexID);
		VertexPositions.Set(VertexID, Points[Idx]);

		if (ImportPointIndex.IsValid())
		{
			ImportPointIndex.Set(VertexID, PointToRawMap[Idx]);
		}
	}

	CopySkinWeightsToAttribute(InSkeletalMesh, NAME_None, Influences, VertexIDMap, nullptr, VertexSkinWeights);

	// Set Bone Attributes
	for (int Idx = 0; Idx < RefBonesBinary.Num(); ++Idx)
	{
		const SkeletalMeshImportData::FBone& Bone = RefBonesBinary[Idx];
		
		const FBoneID BoneID = MeshAttributes.CreateBone();
		
		BoneNames.Set(BoneID, FName(Bone.Name));
		BoneParentIndices.Set(BoneID, Bone.ParentIndex);
		BonePoses.Set(BoneID, FTransform(Bone.BonePos.Transform));
	}

	// Partition the faces by material index. Each material index corresponds to a polygon group.
	// Here, it's worth doing via indirection, due to the size of each FTriangle object.
	TArray<int32> FaceIndices;
	FaceIndices.SetNum(Faces.Num());
	for (int32 Idx = 0; Idx < FaceIndices.Num(); Idx++) { FaceIndices[Idx] = Idx; }
	FaceIndices.Sort([this](const int32 A, const int32 B)
	{
		return Faces[A].MatIndex < Faces[B].MatIndex;
	});

	TArray<FVertexInstanceID> VertexInstanceIDMap;
	VertexInstanceIDMap.Init(INDEX_NONE, Wedges.Num());

	TArray<FVertexInstanceID> TriangleVertexInstanceIDs;
	TriangleVertexInstanceIDs.SetNum(3);

	TArray<uint32> FaceSmoothingMasks;
	FaceSmoothingMasks.AddZeroed(Faces.Num());

	TArray<FPolygonGroupID> MaterialGroups;
	MaterialGroups.Reserve(Materials.Num());
	if (!Materials.IsEmpty())
	{
		for (int32 MaterialIndex = 0; MaterialIndex < Materials.Num(); MaterialIndex++)
		{
			const FPolygonGroupID PolygonGroupID = OutMeshDescription.CreatePolygonGroup();
			PolygonGroupMaterialSlotNames.Set(PolygonGroupID, FName(*Materials[MaterialIndex].MaterialImportName));
			MaterialGroups.Add(PolygonGroupID);
		}
	}
	else
	{
		const FPolygonGroupID PolygonGroupID = OutMeshDescription.CreatePolygonGroup();
		PolygonGroupMaterialSlotNames.Set(PolygonGroupID, NAME_None);
		MaterialGroups.Add(PolygonGroupID);
	}

	for (int32 TriangleIndex = 0; TriangleIndex < Faces.Num(); TriangleIndex++)
	{
		const SkeletalMeshImportData::FTriangle &Triangle = Faces[TriangleIndex];

		for (int32 Corner = 0; Corner < 3; Corner++)
		{
			const uint32 WedgeId = Triangle.WedgeIndex[Corner];
			const SkeletalMeshImportData::FVertex &Wedge = Wedges[WedgeId];
			const FVertexID VertexID = VertexIDMap[Wedge.VertexIndex];

			FVertexInstanceID VertexInstanceID = VertexInstanceIDMap[WedgeId];
			if (VertexInstanceID == INDEX_NONE)
			{
				VertexInstanceID = OutMeshDescription.CreateVertexInstance(VertexID);

				if (bHasVertexColors)
				{
					// Don't perform sRGB conversion (which mirrors what CreateFromMeshDescription does).
					VertexInstanceColors.Set(VertexInstanceID, Wedge.Color.ReinterpretAsLinear());
				}
				else
				{
					// The default color for import data is black. 
					VertexInstanceColors.Set(VertexInstanceID, FLinearColor::Black);
				}
				for (int32 UVIndex = 0; UVIndex < static_cast<int32>(NumTexCoords); UVIndex++)
				{
					VertexInstanceUVs.Set(VertexInstanceID, UVIndex, Wedge.UVs[UVIndex]);
				}

				if (bHasNormals)
				{
					VertexInstanceNormals.Set(VertexInstanceID, Triangle.TangentZ[Corner]);
				}
				if (bHasTangents)
				{
					VertexInstanceTangents.Set(VertexInstanceID, Triangle.TangentX[Corner]);

					// We can only divine the bi-tangent sign if the normal is also given. 
					if (bHasNormals)
					{
						VertexInstanceBinormalSigns.Set(VertexInstanceID,
							((Triangle.TangentZ[Corner] ^ Triangle.TangentX[Corner]) | Triangle.TangentY[Corner]) < 0 ? -1.0f : 1.0f);
					}
				}
				
				VertexInstanceIDMap[WedgeId] = VertexInstanceID;
			}

			TriangleVertexInstanceIDs[Corner] = VertexInstanceID; 
		}

		FPolygonGroupID PolygonGroupID;
		if (MaterialGroups.IsValidIndex(Triangle.MatIndex))
		{
			PolygonGroupID = MaterialGroups[Triangle.MatIndex];
		}
		else
		{
			PolygonGroupID = MaterialGroups[0];
		}
		const FTriangleID TriangleID = OutMeshDescription.CreateTriangle(PolygonGroupID, TriangleVertexInstanceIDs);
		const FPolygonID PolygonID = OutMeshDescription.GetTrianglePolygon(TriangleID);

		if (PolygonID.GetValue() >= FaceSmoothingMasks.Num())
		{
			FaceSmoothingMasks.SetNum(PolygonID.GetValue() + 1);
		}
		FaceSmoothingMasks[PolygonID.GetValue()] = Triangle.SmoothingGroups;
	}

	// Convert morph targets
	for (int32 MorphTargetIndex = 0; MorphTargetIndex < MorphTargets.Num(); MorphTargetIndex++)
	{
		if (!ValidMorphTargets.Contains(MorphTargetIndex))
		{
			continue;
		}

		FString MorphTargetName(ValidMorphTargets[MorphTargetIndex]);
		const TArray<FVector3f>& MorphPoints = MorphTargets[MorphTargetIndex].Points;
		const TSet<uint32>& ModifiedPoints = MorphTargetModifiedPoints[MorphTargetIndex];

		TVertexAttributesRef<FVector3f> PositionDelta = MeshAttributes.GetVertexMorphPositionDelta(*MorphTargetName);

		int32 ModifiedPointIndex = 0;
		// This relies on the dubious assumptions that TSet values are retained in insertion order.
		for (uint32 PointIndex: ModifiedPoints)
		{
			const FVertexID VertexID = VertexIDMap[PointIndex];
			
			PositionDelta.Set(VertexID, MorphPoints[ModifiedPointIndex] - Points[PointIndex]);
			ModifiedPointIndex++;
		}
	}

	// Convert optional skin weights
	for (int32 AlternateInfluenceIndex = 0; AlternateInfluenceIndex < AlternateInfluences.Num(); AlternateInfluenceIndex++)
	{
		FString AlternateInfluenceProfileName(AlternateInfluenceProfileNames[AlternateInfluenceIndex]);
		if (!ValidSkinWeights.Contains(AlternateInfluenceProfileName))
		{
			continue;
		}

		CopySkinWeightsToMeshDescription(InSkeletalMesh, FName(AlternateInfluenceProfileName), AlternateInfluences[AlternateInfluenceIndex], VertexIDMap, OutMeshDescription);
	}
	

	// Convert vertex attributes.
	for (int32 AttributeIndex = 0; AttributeIndex < VertexAttributes.Num(); AttributeIndex++)
	{
		const FString& VertexAttributeName = VertexAttributeNames[AttributeIndex];
		const SkeletalMeshImportData::FVertexAttribute& VertexAttribute = VertexAttributes[AttributeIndex];
		
		if (!ValidAttributes.Contains(VertexAttributeName))
		{
			continue;
		}
		
		FName RegisteredName(VertexAttributeName);
		switch(VertexAttribute.ComponentCount)
		{
		case 1:
			{
				TVertexAttributesRef<float> AttributeRef = OutMeshDescription.VertexAttributes().GetAttributesRef<float>(RegisteredName);
				for (int32 Index = 0; Index < VertexAttribute.AttributeValues.Num(); Index++)
				{
					AttributeRef.Set(VertexIDMap[Index], VertexAttribute.AttributeValues[Index]);
				}
				break;
			}
		case 2:
			{
				TVertexAttributesRef<FVector2f> AttributeRef = OutMeshDescription.VertexAttributes().GetAttributesRef<FVector2f>(RegisteredName);
				for (int32 Index = 0; Index < VertexAttribute.AttributeValues.Num(); Index += 2)
				{
					AttributeRef.Set(VertexIDMap[Index / 2],
						FVector2f(VertexAttribute.AttributeValues[Index], VertexAttribute.AttributeValues[Index + 1]));
				}
				break;
			}
		case 3:
			{
				TVertexAttributesRef<FVector3f> AttributeRef = OutMeshDescription.VertexAttributes().GetAttributesRef<FVector3f>(RegisteredName);
				for (int32 Index = 0; Index < VertexAttribute.AttributeValues.Num(); Index += 3)
				{
					AttributeRef.Set(VertexIDMap[Index / 3],
						FVector3f(VertexAttribute.AttributeValues[Index], VertexAttribute.AttributeValues[Index + 1], VertexAttribute.AttributeValues[Index + 2]));
				}
				break;
			}
		case 4:
			{
				TVertexAttributesRef<FVector4f> AttributeRef = OutMeshDescription.VertexAttributes().GetAttributesRef<FVector4f>(RegisteredName);
				for (int32 Index = 0; Index < VertexAttribute.AttributeValues.Num(); Index += 4)
				{
					AttributeRef.Set(VertexIDMap[Index / 4],
						FVector4f(
							VertexAttribute.AttributeValues[Index], VertexAttribute.AttributeValues[Index + 1],
							VertexAttribute.AttributeValues[Index + 2], VertexAttribute.AttributeValues[Index + 3]));
				}
				break;
			}
		default:
			checkNoEntry();
		}
	}

	// We don't need to do this when retrieving alt skin weights.
	FString SkeletalMeshPath;
	if (InSkeletalMesh)
	{
		SkeletalMeshPath = InSkeletalMesh->GetOuter()->GetPathName();		
	}
	
	FSkeletalMeshOperations::ConvertSmoothGroupToHardEdges(FaceSmoothingMasks, OutMeshDescription);

	// Check if we have any broken data, including UVs and normals/tangents.
	FSkeletalMeshOperations::ValidateAndFixData(OutMeshDescription, SkeletalMeshPath);

	bool bHasInvalidNormals, bHasInvalidTangents;
	FStaticMeshOperations::HasInvalidVertexInstanceNormalsOrTangents(OutMeshDescription, bHasInvalidNormals, bHasInvalidTangents);
	if (bHasInvalidNormals || bHasInvalidTangents)
	{
		// This is required by FSkeletalMeshOperations::ComputeTangentsAndNormals to function correctly.
		FSkeletalMeshOperations::ComputeTriangleTangentsAndNormals(OutMeshDescription, UE_SMALL_NUMBER, !SkeletalMeshPath.IsEmpty() ? *SkeletalMeshPath : nullptr);

		EComputeNTBsFlags ComputeNTBsOptions = EComputeNTBsFlags::None;
		if (InBuildSettings && InBuildSettings->bComputeWeightedNormals)
		{
			ComputeNTBsOptions |= EComputeNTBsFlags::WeightedNTBs;
		}

		// This only recomputes broken normals/tangents. The ValidateAndFixData function above will have turned all non-finite normals and tangents into
		// zero vectors.
		FSkeletalMeshOperations::ComputeTangentsAndNormals(OutMeshDescription, ComputeNTBsOptions);

		// We don't need the triangle tangents and normals anymore.
		OutMeshDescription.TriangleAttributes().UnregisterAttribute(MeshAttribute::Triangle::Normal);
		OutMeshDescription.TriangleAttributes().UnregisterAttribute(MeshAttribute::Triangle::Tangent);
		OutMeshDescription.TriangleAttributes().UnregisterAttribute(MeshAttribute::Triangle::Binormal);
	}

	// Convert the MeshInfo data.
	if (!MeshInfos.IsEmpty())
	{
		FSkeletalMeshAttributes::FSourceGeometryPartNameRef NameAttribute = MeshAttributes.GetSourceGeometryPartNames();
		FSkeletalMeshAttributes::FSourceGeometryPartVertexOffsetAndCountRef VertexAndCountAttribute = MeshAttributes.GetSourceGeometryPartVertexOffsetAndCounts();

		MeshAttributes.SourceGeometryParts().Reserve(MeshInfos.Num());
		
		for (const SkeletalMeshImportData::FMeshInfo& Info: MeshInfos)
		{
			FSourceGeometryPartID PartID = MeshAttributes.CreateSourceGeometryPart();

			NameAttribute.Set(PartID, Info.Name);
			VertexAndCountAttribute.Set(PartID, {Info.StartImportedVertex, Info.NumVertices});
		}
	}

	OutMeshDescription.BuildIndexers();
	
	return true;
}


void FSkeletalMeshImportData::CopySkinWeightsToMeshDescription(
	const USkeletalMesh* InSkeletalMesh,
	const FName InSkinWeightName, 
	const FSkeletalMeshImportData& InSkinWeightMesh,
	const TArray<FVertexID>& InVertexIDMap,
	FMeshDescription& OutMeshDescription
	) const
{
	// The invariant here is that the point count is the same as on the base mesh.

	// - First we check if the topology is the same. For some reason, the two meshes could have been triangulated differently on 
	//   import, in which case we will follow the next set of steps. However, if the topology is exactly the same, we just assume  
	//   that if there are point position differences, they're caused by subtle deformations during output, rather than some
	//   explicit differences.
	// - If the topologies differ, we use a closest-triangle method implemented in FSkeletalMeshOperations::CopySkinWeightAttributeFromMesh.
	FSkeletalMeshAttributes MeshAttributes(OutMeshDescription);
	FSkinWeightsVertexAttributesRef VertexSkinWeights = MeshAttributes.GetVertexSkinWeights(InSkinWeightName);

	// Create a mapping from the alt influence mesh bones to the base mesh bones.
	TMap<FString, int32> BaseBoneToIndexMap;
	for (int32 BoneIndex = 0; BoneIndex < RefBonesBinary.Num(); BoneIndex++)
	{
		BaseBoneToIndexMap.Add(RefBonesBinary[BoneIndex].Name, BoneIndex);
	}
	TMap<int32, int32> AltMeshBoneToBaseBoneMap;
	for (int32 BoneIndex = 0; BoneIndex < InSkinWeightMesh.RefBonesBinary.Num(); BoneIndex++)
	{
		// Any bone not in the base mesh will not get bound. If that results in no bindings for a vertex, the vertex will be bound to the root.
		if (const int32* BaseBoneIndex = BaseBoneToIndexMap.Find(InSkinWeightMesh.RefBonesBinary[BoneIndex].Name))
		{
			AltMeshBoneToBaseBoneMap.Add(BoneIndex, *BaseBoneIndex);
		}
	}
	
	bool bTopologySame = true;
	if (InSkinWeightMesh.Faces.Num() == Faces.Num())
	{
		for (int32 FaceIndex = 0; FaceIndex < Faces.Num(); FaceIndex++)
		{
			if (Faces[FaceIndex].WedgeIndex[0] != InSkinWeightMesh.Faces[FaceIndex].WedgeIndex[0] ||
				Faces[FaceIndex].WedgeIndex[1] != InSkinWeightMesh.Faces[FaceIndex].WedgeIndex[1] ||
				Faces[FaceIndex].WedgeIndex[2] != InSkinWeightMesh.Faces[FaceIndex].WedgeIndex[2])
			{
				bTopologySame = false;
				break;
			}
		}
	}
	else
	{
		bTopologySame = false;
	}

	if (bTopologySame && InSkinWeightMesh.Wedges.Num() == Wedges.Num())
	{
		for (int32 WedgeIndex = 0; WedgeIndex < Wedges.Num(); WedgeIndex++)
		{
			if (Wedges[WedgeIndex].VertexIndex != InSkinWeightMesh.Wedges[WedgeIndex].VertexIndex)
			{
				bTopologySame = false;
				break;
			}
		}
	}
	else
	{
		bTopologySame = false;
	}

	if (bTopologySame)
	{
		CopySkinWeightsToAttribute(InSkeletalMesh, InSkinWeightName, InSkinWeightMesh.Influences, InVertexIDMap, &AltMeshBoneToBaseBoneMap, VertexSkinWeights);
		return;
	}

	// The topologies don't match, proceed as above.
	FMeshDescription AlternateInfluenceMesh;
	InSkinWeightMesh.GetMeshDescription(nullptr, nullptr, AlternateInfluenceMesh);
	
	FSkeletalMeshOperations::CopySkinWeightAttributeFromMesh(
	AlternateInfluenceMesh, OutMeshDescription, NAME_None, InSkinWeightName, &AltMeshBoneToBaseBoneMap);   
}


// A simpler variant of FStaticMeshOperations::ConvertHardEdgesToSmoothGroup that assumes that hard edges always form closed regions.
static void ConvertHardEdgesToSmoothMasks(
	const FMeshDescription& InMeshDescription,
	TArray<uint32>& OutSmoothMasks
	)
{
	OutSmoothMasks.SetNumZeroed(InMeshDescription.Triangles().Num());

	TSet<FTriangleID> ProcessedTriangles;
	TArray<FTriangleID> TriangleQueue;
	uint32 CurrentSmoothMask = 1;

	const TEdgeAttributesConstRef<bool> IsEdgeHard = InMeshDescription.EdgeAttributes().GetAttributesRef<bool>(MeshAttribute::Edge::IsHard);	

	for (FTriangleID SeedTriangleID: InMeshDescription.Triangles().GetElementIDs())
	{
		if (ProcessedTriangles.Contains(SeedTriangleID))
		{
			continue;
		}
		
		TriangleQueue.Push(SeedTriangleID);
		while (!TriangleQueue.IsEmpty())
		{
			const FTriangleID TriangleID = TriangleQueue.Pop(EAllowShrinking::No);
			TArrayView<const FEdgeID> TriangleEdges = InMeshDescription.GetTriangleEdges(TriangleID);

			OutSmoothMasks[TriangleID.GetValue()] = CurrentSmoothMask;
			ProcessedTriangles.Add(TriangleID);

			for (const FEdgeID EdgeID: TriangleEdges)
			{
				if (!IsEdgeHard.Get(EdgeID))
				{
					TArrayView<const FTriangleID> ConnectedTriangles = InMeshDescription.GetEdgeConnectedTriangleIDs(EdgeID);
					for (const FTriangleID NeighborTriangleID: ConnectedTriangles)
					{
						if (!ProcessedTriangles.Contains(NeighborTriangleID))
						{
							TriangleQueue.Push(NeighborTriangleID);
						}
					}
				}
			}
		}

		CurrentSmoothMask <<= 1;
		if (CurrentSmoothMask == 0)
		{
			// If we exhausted all available bits, then thunk to the more complete algorithm. For reasons unknown at this time, it doesn't generate
			// nice smooth groups for some simpler test objects. For more complex input products it does a decent job though.
			OutSmoothMasks.SetNumZeroed(InMeshDescription.Triangles().Num());
			FStaticMeshOperations::ConvertHardEdgesToSmoothGroup(InMeshDescription, OutSmoothMasks);
			break;
		}
	}
}




template<typename T>
struct FCopyAttributeElement {};

template<>
struct FCopyAttributeElement<float>
{
	static const int32 ComponentCount = 1;
	static void CopyElement(const float InValue, TArrayView<float> OutValue)
	{
		OutValue[0] = InValue;
	}
};

template<>
struct FCopyAttributeElement<FVector2f>
{
	static const int32 ComponentCount = 2;
	static void CopyElement(const FVector2f& InValue, TArrayView<float> OutValue)
	{
		OutValue[0] = InValue.X;
		OutValue[1] = InValue.Y;
	}
};

template<>
struct FCopyAttributeElement<FVector3f>
{
	static const int32 ComponentCount = 3;
	static void CopyElement(const FVector3f& InValue, TArrayView<float> OutValue)
	{
		OutValue[0] = InValue.X;
		OutValue[1] = InValue.Y;
		OutValue[2] = InValue.Z;
	}
};

template<>
struct FCopyAttributeElement<FVector4f>
{
	static const int32 ComponentCount = 4;
	static void CopyElement(const FVector4f& InValue, TArrayView<float> OutValue)
	{
		OutValue[0] = InValue.X;
		OutValue[1] = InValue.Y;
		OutValue[2] = InValue.Z;
		OutValue[3] = InValue.W;
	}
};


template<typename T>
struct FCreateAndCopyAttributeValues
{
	FCreateAndCopyAttributeValues(
		FSkeletalMeshImportData& InImportData,
		const FVertexArray& InVertices ) :
	ImportData(InImportData), Vertices(InVertices) {}
	
	void operator()(const FName InAttributeName, TVertexAttributesConstRef<T> InSrcAttribute)
	{
		if (FSkeletalMeshAttributes::IsReservedAttributeName(InAttributeName))
		{
			return;
		}

		// We should not end up with a naming clash, but err on the side of caution,
		if (!ensure(!ImportData.VertexAttributeNames.Contains(InAttributeName.ToString())))
		{
			return;
		}

		ImportData.VertexAttributeNames.Add(InAttributeName.ToString());
		SkeletalMeshImportData::FVertexAttribute& DstAttribute = ImportData.VertexAttributes.AddDefaulted_GetRef();
		
		DstAttribute.ComponentCount = FCopyAttributeElement<T>::ComponentCount;
		DstAttribute.AttributeValues.SetNumUninitialized(ImportData.Points.Num());
		for (int32 VertexIndex = 0; VertexIndex < ImportData.Points.Num(); VertexIndex++)
		{
			// We go via the PointToRawMap because the number of points could have increased after a call to 
			// SplitVerticesBySmoothingGroups.
			FVertexID VertexID(ImportData.PointToRawMap[VertexIndex]);
			FCopyAttributeElement<T>::CopyElement(
				InSrcAttribute.Get(VertexID),
				TArrayView<float>(DstAttribute.AttributeValues.GetData() + VertexIndex * DstAttribute.ComponentCount, DstAttribute.ComponentCount));
		}
	}
	
	// Unhandled sub-types.
	void operator()(const FName, TVertexAttributesConstRef<TArrayAttribute<T>>) { }
	void operator()(const FName, TVertexAttributesConstRef<TArrayView<T>>) { }

private:
	FSkeletalMeshImportData& ImportData;
	const FVertexArray& Vertices;
};


FSkeletalMeshImportData FSkeletalMeshImportData::CreateFromMeshDescription(const FMeshDescription& InMeshDescription)
{
	FSkeletalMeshImportData SkelMeshImportData;
	
	FSkeletalMeshConstAttributes MeshAttributes(InMeshDescription);
	TVertexAttributesConstRef<FVector3f> VertexPositions = MeshAttributes.GetVertexPositions();
	FSkinWeightsVertexAttributesConstRef VertexSkinWeights = MeshAttributes.GetVertexSkinWeights();

	TVertexAttributesConstRef<int32> ImportPointIndex = InMeshDescription.VertexAttributes().GetAttributesRef<int32>(MeshAttribute::Vertex::ImportPointIndex); 
		
	TVertexInstanceAttributesConstRef<FVector2f> VertexInstanceUVs = MeshAttributes.GetVertexInstanceUVs();
	TVertexInstanceAttributesConstRef<FVector3f> VertexInstanceNormals = MeshAttributes.GetVertexInstanceNormals();
	TVertexInstanceAttributesConstRef<FVector3f> VertexInstanceTangents = MeshAttributes.GetVertexInstanceTangents();
	TVertexInstanceAttributesConstRef<float> VertexInstanceBiNormalSigns = MeshAttributes.GetVertexInstanceBinormalSigns();
	TVertexInstanceAttributesConstRef<FVector4f> VertexInstanceColors = MeshAttributes.GetVertexInstanceColors();

	TPolygonGroupAttributesConstRef<FName> PolygonGroupMaterialSlotNames = MeshAttributes.GetPolygonGroupMaterialSlotNames();

	//Get the per face smoothing
	TArray<uint32> FaceSmoothingMasks;
	ConvertHardEdgesToSmoothMasks(InMeshDescription, FaceSmoothingMasks);

	//////////////////////////////////////////////////////////////////////////
	// Copy the materials
	SkelMeshImportData.Materials.Reserve(InMeshDescription.PolygonGroups().Num());
	for (FPolygonGroupID PolygonGroupID : InMeshDescription.PolygonGroups().GetElementIDs())
	{
		SkeletalMeshImportData::FMaterial Material;
		Material.MaterialImportName = PolygonGroupMaterialSlotNames[PolygonGroupID].ToString();
		//The material interface will be added later by the factory
		SkelMeshImportData.Materials.Add(Material);
	}
	SkelMeshImportData.MaxMaterialIndex = SkelMeshImportData.Materials.Num()-1;

	//////////////////////////////////////////////////////////////////////////
	//Copy the vertex positions and the influences

	//Reserve the point and influences
	SkelMeshImportData.Points.SetNumUninitialized(InMeshDescription.Vertices().GetArraySize());
	SkelMeshImportData.Influences.Reserve(InMeshDescription.Vertices().GetArraySize() * 4);
	SkelMeshImportData.PointToRawMap.SetNumUninitialized(InMeshDescription.Vertices().GetArraySize());

	for (FVertexID VertexID : InMeshDescription.Vertices().GetElementIDs())
	{
		SkelMeshImportData.Points[VertexID.GetValue()] = VertexPositions[VertexID];
		if (VertexSkinWeights.IsValid())
		{
			FVertexBoneWeightsConst BoneWeights = VertexSkinWeights.Get(VertexID);
			const int32 InfluenceCount = BoneWeights.Num();

			const int32 InfluenceOffsetIndex = SkelMeshImportData.Influences.Num();
			SkelMeshImportData.Influences.AddDefaulted(InfluenceCount);
			for (int32 InfluenceIndex = 0; InfluenceIndex < InfluenceCount; ++InfluenceIndex)
			{
				SkeletalMeshImportData::FRawBoneInfluence& BoneInfluence = SkelMeshImportData.Influences[InfluenceOffsetIndex + InfluenceIndex];
				BoneInfluence.VertexIndex = VertexID.GetValue();
				BoneInfluence.BoneIndex = BoneWeights[InfluenceIndex].GetBoneIndex();
				BoneInfluence.Weight = BoneWeights[InfluenceIndex].GetWeight();
			}
		}

		if (ImportPointIndex.IsValid())
		{
			SkelMeshImportData.PointToRawMap[VertexID.GetValue()] = ImportPointIndex[VertexID] == INDEX_NONE ? VertexID.GetValue() : ImportPointIndex[VertexID];
		}
		else
		{
			SkelMeshImportData.PointToRawMap[VertexID.GetValue()] = VertexID.GetValue();
		}
	}

	bool bHaveValidNormals = false;
	bool bHaveValidTangents = false;
	for (FVertexInstanceID VertexInstanceID: InMeshDescription.VertexInstances().GetElementIDs())
	{
		const FVector3f Normal = VertexInstanceNormals.Get(VertexInstanceID);
		if (!Normal.IsNearlyZero(UE_SMALL_NUMBER) && !Normal.ContainsNaN())
		{
			bHaveValidNormals = true;
		}
		const FVector3f Tangent = VertexInstanceTangents.Get(VertexInstanceID);
		if (!Tangent.IsNearlyZero(UE_SMALL_NUMBER) && !Tangent.ContainsNaN())
		{
			bHaveValidTangents = true;
		}

		// If we found any valid normals/tangents, we can stop now.
		if (bHaveValidNormals && bHaveValidTangents)
		{
			break;
		}
	}

	//////////////////////////////////////////////////////////////////////////
	// Copy vertex instances into wedges.
	SkelMeshImportData.NumTexCoords = FMath::Min<int32>(VertexInstanceUVs.GetNumChannels(), (int32)MAX_TEXCOORDS);
	SkelMeshImportData.bHasNormals = bHaveValidNormals;
	SkelMeshImportData.bHasTangents = bHaveValidTangents;
	
	SkelMeshImportData.Wedges.SetNumZeroed(InMeshDescription.VertexInstances().GetArraySize());
	
	auto FillWedge = [&MeshDescription=InMeshDescription, &SkelMeshImportData, &VertexInstanceColors, &VertexInstanceUVs](const FVertexInstanceID InVertexInstanceID, SkeletalMeshImportData::FVertex& OutWedge)
	{
		OutWedge.VertexIndex = static_cast<uint32>(MeshDescription.GetVertexInstanceVertex(InVertexInstanceID).GetValue());
		OutWedge.MatIndex = 0;			// We set this later -- not that this is actually used by any internal process.
		
		constexpr bool bSRGB = false; //avoid linear to srgb conversion
		const FLinearColor VertexColor = VertexInstanceColors[InVertexInstanceID];
		if (VertexColor != FLinearColor::Black)
		{
			SkelMeshImportData.bHasVertexColors = true;
		}
		OutWedge.Color = VertexColor.ToFColor(bSRGB);
		
		for (int32 UVChannelIndex = 0; UVChannelIndex < static_cast<int32>(SkelMeshImportData.NumTexCoords); ++UVChannelIndex)
		{
			OutWedge.UVs[UVChannelIndex] = VertexInstanceUVs.Get(InVertexInstanceID, UVChannelIndex);
		}
	};
	
	for (FVertexInstanceID VertexInstanceID: InMeshDescription.VertexInstances().GetElementIDs())
	{
		SkeletalMeshImportData::FVertex& Wedge = SkelMeshImportData.Wedges[VertexInstanceID.GetValue()];

		FillWedge(VertexInstanceID, Wedge);
	}
	
	// Keep track of which vertex instance ID has been used already. If it gets used again, make a duplicate, so that
	// we don't end up with two wedges being used by the same face. Downstream algorithms do not like it.
	// We have to do ensure that the vertex instances are kept in the same order that they got defined originally, because
	// there are some models that add externally defined morph targets, and they require that the point order is kept
	// consistent.
	TBitArray UsedWedges;
	UsedWedges.SetNum(InMeshDescription.VertexInstances().GetArraySize(), false);
	
	//////////////////////////////////////////////////////////////////////////
	// Copy the triangles
	SkelMeshImportData.Faces.SetNumZeroed(InMeshDescription.Triangles().GetArraySize());
	for (FTriangleID TriangleID : InMeshDescription.Triangles().GetElementIDs())
	{
		FPolygonGroupID PolygonGroupID = InMeshDescription.GetTrianglePolygonGroup(TriangleID);
		TArrayView<const FVertexInstanceID> VertexInstances = InMeshDescription.GetTriangleVertexInstances(TriangleID);
		int32 TriangleIndex = TriangleID.GetValue();
		
		SkeletalMeshImportData::FTriangle& Face = SkelMeshImportData.Faces[TriangleIndex];
		Face.MatIndex = PolygonGroupID.GetValue();
		Face.SmoothingGroups = 0;
		if (FaceSmoothingMasks.IsValidIndex(TriangleIndex))
		{
			Face.SmoothingGroups = FaceSmoothingMasks[TriangleIndex];
		}
		for (int32 Corner = 0; Corner < 3; ++Corner)
		{
			FVertexInstanceID VertexInstanceID = VertexInstances[Corner];
			
			if (bHaveValidNormals)
			{
				Face.TangentZ[Corner] = VertexInstanceNormals[VertexInstanceID];
			}
			else
			{
				// The normal/tangent computation during conversion to render data will automatically regenerate any degenerate normals.
				// Same for the tangent/binormal below.
				Face.TangentZ[Corner] = FVector3f::ZeroVector;
			}

			if (bHaveValidTangents && bHaveValidNormals)
			{
				Face.TangentX[Corner] = VertexInstanceTangents[VertexInstanceID];
				Face.TangentY[Corner] = FVector3f::CrossProduct(VertexInstanceNormals[VertexInstanceID], VertexInstanceTangents[VertexInstanceID]).GetSafeNormal() * VertexInstanceBiNormalSigns[VertexInstanceID];
			}
			else
			{
				Face.TangentX[Corner] = FVector3f::ZeroVector;
				Face.TangentY[Corner] = FVector3f::ZeroVector;
			}

			int32 WedgeIndex = VertexInstanceID.GetValue();
			if (!UsedWedges[WedgeIndex])
			{
				// This wedge has not been used before, mark it for first use, so that any subsequent use can be
				// used to form a duplicate.
				UsedWedges[WedgeIndex] = true;
			}
			else
			{
				// Create a new, duplicate wedge.
				WedgeIndex = SkelMeshImportData.Wedges.Num();
				FillWedge(VertexInstanceID, SkelMeshImportData.Wedges.AddZeroed_GetRef());
			}
			
			Face.WedgeIndex[Corner] = WedgeIndex;
			SkelMeshImportData.Wedges[WedgeIndex].MatIndex = Face.MatIndex;
		}
	}

	// Update Bones data
	const FSkeletalMeshAttributes::FBoneArray& Bones = MeshAttributes.Bones();
	SkelMeshImportData.RefBonesBinary.Reserve(Bones.Num());
	
	FSkeletalMeshAttributes::FBoneNameAttributesConstRef BoneNames = MeshAttributes.GetBoneNames();
	FSkeletalMeshAttributes::FBoneParentIndexAttributesConstRef BoneParentIndices = MeshAttributes.GetBoneParentIndices();
	FSkeletalMeshAttributes::FBonePoseAttributesConstRef BonePoses = MeshAttributes.GetBonePoses();
	if (BoneNames.IsValid() && BoneParentIndices.IsValid() && BonePoses.IsValid())
	{
		TArray<int32> NumChildren;
		NumChildren.Init(0, Bones.Num());
		
		for (const FBoneID& BoneId : Bones.GetElementIDs())
		{
			const int32 ParentIndex = BoneParentIndices.Get(BoneId);
			if (ParentIndex != INDEX_NONE)
			{
				++NumChildren[ParentIndex];
			}
		}

		int32 Index = 0;
		for (const FBoneID& BoneId : Bones.GetElementIDs())
		{
			SkeletalMeshImportData::FBone NewBone;
			NewBone.Name = BoneNames.Get(BoneId).ToString();
			NewBone.ParentIndex = BoneParentIndices.Get(BoneId);
			NewBone.BonePos.Transform = FTransform3f(BonePoses.Get(BoneId));
			NewBone.NumChildren = NumChildren[Index];
			SkelMeshImportData.RefBonesBinary.Emplace(MoveTemp(NewBone));
			Index++;
		}
	}

	// Copy morph targets.
	for (const FName MorphTargetName: MeshAttributes.GetMorphTargetNames())
	{
		TVertexAttributesConstRef<FVector3f> MorphTargetPosDeltaAttribute = MeshAttributes.GetVertexMorphPositionDelta(MorphTargetName);

		SkelMeshImportData.MorphTargetNames.Add(MorphTargetName.ToString());
		FSkeletalMeshImportData& MorphTarget = SkelMeshImportData.MorphTargets.AddDefaulted_GetRef();
		TSet<uint32>& MorphTargetModifiedPoints = SkelMeshImportData.MorphTargetModifiedPoints.AddDefaulted_GetRef();

		for (FVertexID VertexID : InMeshDescription.Vertices().GetElementIDs())
		{
			FVector3f PositionDelta = MorphTargetPosDeltaAttribute.Get(VertexID);
			if (!PositionDelta.IsNearlyZero())
			{
				MorphTarget.Points.Add(PositionDelta + SkelMeshImportData.Points[VertexID.GetValue()]);
				MorphTargetModifiedPoints.Add(VertexID.GetValue());
			}
		}
	}

	// Copy alternate influences.
	for (const FName SkinWeightProfileName: MeshAttributes.GetSkinWeightProfileNames())
	{
		// The default profile was already handled above.
		if (SkinWeightProfileName == FSkeletalMeshAttributes::DefaultSkinWeightProfileName)
		{
			continue;
		}
		
		FSkinWeightsVertexAttributesConstRef SkinWeightsAttribute = MeshAttributes.GetVertexSkinWeights(SkinWeightProfileName);

		SkelMeshImportData.AlternateInfluenceProfileNames.Add(SkinWeightProfileName.ToString());
		FSkeletalMeshImportData& AlternateInfluenceMesh = SkelMeshImportData.AlternateInfluences.AddDefaulted_GetRef();
		
		// Copy over all the relevant mesh data aside from influences themselves.
		AlternateInfluenceMesh.Points = SkelMeshImportData.Points;
		AlternateInfluenceMesh.Wedges = SkelMeshImportData.Wedges;
		AlternateInfluenceMesh.Faces = SkelMeshImportData.Faces;
		AlternateInfluenceMesh.RefBonesBinary = SkelMeshImportData.RefBonesBinary;

		for (FVertexID VertexID : InMeshDescription.Vertices().GetElementIDs())
		{
			FVertexBoneWeightsConst BoneWeights = SkinWeightsAttribute.Get(VertexID);
			const int32 InfluenceCount = BoneWeights.Num();

			const int32 InfluenceOffsetIndex = AlternateInfluenceMesh.Influences.Num();
			AlternateInfluenceMesh.Influences.AddDefaulted(InfluenceCount);
			for (int32 InfluenceIndex = 0; InfluenceIndex < InfluenceCount; ++InfluenceIndex)
			{
				SkeletalMeshImportData::FRawBoneInfluence& BoneInfluence = AlternateInfluenceMesh.Influences[InfluenceOffsetIndex + InfluenceIndex];
				BoneInfluence.VertexIndex = VertexID.GetValue();
				BoneInfluence.BoneIndex = BoneWeights[InfluenceIndex].GetBoneIndex();
				BoneInfluence.Weight = BoneWeights[InfluenceIndex].GetWeight();
			}
		}
	}
	
	// Copy any non-reserved float vertex attributes.
	InMeshDescription.VertexAttributes().ForEachByType<float>(FCreateAndCopyAttributeValues<float>(SkelMeshImportData, InMeshDescription.Vertices()));
	InMeshDescription.VertexAttributes().ForEachByType<FVector2f>(FCreateAndCopyAttributeValues<FVector2f>(SkelMeshImportData, InMeshDescription.Vertices()));
	InMeshDescription.VertexAttributes().ForEachByType<FVector3f>(FCreateAndCopyAttributeValues<FVector3f>(SkelMeshImportData, InMeshDescription.Vertices()));
	InMeshDescription.VertexAttributes().ForEachByType<FVector4f>(FCreateAndCopyAttributeValues<FVector4f>(SkelMeshImportData, InMeshDescription.Vertices()));

	// Add in the geometry part data as mesh info. We don't particularly care about validity of the data, leave that for the user.
	// FIXME: This should be stored as a polygroup on the mesh description object instead.
	if (MeshAttributes.HasSourceGeometryParts())
	{
		FSkeletalMeshAttributes::FSourceGeometryPartNameConstRef NameAttribute = MeshAttributes.GetSourceGeometryPartNames();
		FSkeletalMeshAttributes::FSourceGeometryPartVertexOffsetAndCountConstRef VertexAndCountAttribute = MeshAttributes.GetSourceGeometryPartVertexOffsetAndCounts();

		for (FSourceGeometryPartID SourceGeometryPartID: MeshAttributes.SourceGeometryParts().GetElementIDs())
		{
			SkeletalMeshImportData::FMeshInfo Info;
			Info.Name = NameAttribute.Get(SourceGeometryPartID);
			Info.NumVertices = VertexAndCountAttribute.Get(SourceGeometryPartID)[1];
			Info.StartImportedVertex = VertexAndCountAttribute.Get(SourceGeometryPartID)[0]; 
			SkelMeshImportData.MeshInfos.Add(Info);
		}
	}

	return SkelMeshImportData;
}

#endif // WITH_EDITOR
