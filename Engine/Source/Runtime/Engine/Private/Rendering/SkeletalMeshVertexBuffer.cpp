// Copyright Epic Games, Inc. All Rights Reserved.

#include "Rendering/SkeletalMeshVertexBuffer.h"
#include "Containers/ClosableMpscQueue.h"
#include "EngineUtils.h"
#include "EngineLogs.h"
#include "Experimental/Containers/HazardPointer.h"
#include "RHI.h"
#include "SkeletalMeshLegacyCustomVersions.h"

/**
* Destructor
*/
FDummySkeletalMeshVertexBuffer::~FDummySkeletalMeshVertexBuffer()
{
	CleanUp();
}

/**
* Delete existing resources
*/
void FDummySkeletalMeshVertexBuffer::CleanUp()
{
	delete VertexData;
	VertexData = nullptr;
}


/**
* Serializer for this class
* @param Ar - archive to serialize to
* @param B - data to serialize
*/
FArchive& operator<<(FArchive& Ar, FDummySkeletalMeshVertexBuffer& VertexBuffer)
{
	FStripDataFlags StripFlags(Ar, 0, FPackageFileVersion::CreateUE4Version(VER_UE4_STATIC_SKELETAL_MESH_SERIALIZATION_FIX));

	Ar << VertexBuffer.NumTexCoords;
	Ar << VertexBuffer.bUseFullPrecisionUVs;

	bool bBackCompatExtraBoneInfluences = false;

	Ar.UsingCustomVersion(FSkeletalMeshCustomVersion::GUID);

	if (Ar.UEVer() >= VER_UE4_SUPPORT_GPUSKINNING_8_BONE_INFLUENCES && Ar.CustomVer(FSkeletalMeshCustomVersion::GUID) < FSkeletalMeshCustomVersion::UseSeparateSkinWeightBuffer)
	{
		Ar << bBackCompatExtraBoneInfluences;
	}

	// Serialize MeshExtension and Origin
	// I need to save them for console to pick it up later
	FVector Dummy;
	Ar << Dummy << Dummy;

	if (Ar.IsLoading())
	{
		// allocate vertex data on load
		VertexBuffer.AllocateData();
	}

	// if Ar is counting, it still should serialize. Need to count VertexData
	if (!StripFlags.IsAudioVisualDataStripped() || Ar.IsCountingMemory())
	{
		// Special handling for loading old content
		if (Ar.IsLoading() && Ar.CustomVer(FSkeletalMeshCustomVersion::GUID) < FSkeletalMeshCustomVersion::UseSeparateSkinWeightBuffer)
		{
			int32 ElementSize;
			Ar << ElementSize;

			int32 ArrayNum;
			Ar << ArrayNum;

			TArray<uint8> DummyBytes;
			DummyBytes.AddUninitialized(ElementSize * ArrayNum);
			Ar.Serialize(DummyBytes.GetData(), ElementSize * ArrayNum);
		}
		else
		{
			if (VertexBuffer.VertexData != NULL)
			{
				VertexBuffer.VertexData->Serialize(Ar);

				// update cached buffer info
				VertexBuffer.NumVertices = VertexBuffer.VertexData->GetNumVertices();
				VertexBuffer.Data = (VertexBuffer.NumVertices > 0) ? VertexBuffer.VertexData->GetDataPointer() : nullptr;
				VertexBuffer.Stride = VertexBuffer.VertexData->GetStride();
			}
		}
	}

	return Ar;
}

// Handy macro for allocating the correct vertex data class (which has to be known at compile time) depending on the data type and number of UVs.  
#define ALLOCATE_VERTEX_DATA_TEMPLATE( VertexDataType, NumUVs )											\
	switch(NumUVs)																						\
	{																									\
		case 1: VertexData = new TSkeletalMeshVertexData< VertexDataType<1> >(bNeedsCPUAccess); break;	\
		case 2: VertexData = new TSkeletalMeshVertexData< VertexDataType<2> >(bNeedsCPUAccess); break;	\
		case 3: VertexData = new TSkeletalMeshVertexData< VertexDataType<3> >(bNeedsCPUAccess); break;	\
		case 4: VertexData = new TSkeletalMeshVertexData< VertexDataType<4> >(bNeedsCPUAccess); break;	\
		default: UE_LOGF(LogSkeletalMesh, Fatal,"Invalid number of texture coordinates");								\
	}																									\

/**
* Allocates the vertex data storage type.
*/
void FDummySkeletalMeshVertexBuffer::AllocateData()
{
	// Clear any old VertexData before allocating.
	CleanUp();

	if (!bUseFullPrecisionUVs)
	{
		ALLOCATE_VERTEX_DATA_TEMPLATE(TGPUSkinVertexFloat16Uvs, NumTexCoords);
	}
	else
	{
		ALLOCATE_VERTEX_DATA_TEMPLATE(TGPUSkinVertexFloat32Uvs, NumTexCoords);
	}
}
