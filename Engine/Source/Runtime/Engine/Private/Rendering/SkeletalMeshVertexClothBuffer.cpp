// Copyright Epic Games, Inc. All Rights Reserved.

#include "Rendering/SkeletalMeshVertexClothBuffer.h"
#include "Containers/ClosableMpscQueue.h"
#include "Rendering/SkeletalMeshVertexBuffer.h"
#include "EngineUtils.h"
#include "Experimental/Containers/HazardPointer.h"
#include "ProfilingDebugging/LoadTimeTracker.h"
#include "RHIResourceReplace.h"
#include "RHI.h"

/**
* Constructor
*/
FSkeletalMeshVertexClothBuffer::FSkeletalMeshVertexClothBuffer()
	: VertexData(nullptr),
	Data(nullptr),
	Stride(0),
	NumVertices(0)
{

}

/**
* Destructor
*/
FSkeletalMeshVertexClothBuffer::~FSkeletalMeshVertexClothBuffer()
{
	// clean up everything
	CleanUp();
}

/**
* Assignment. Assumes that vertex buffer will be rebuilt
*/

FSkeletalMeshVertexClothBuffer& FSkeletalMeshVertexClothBuffer::operator=(const FSkeletalMeshVertexClothBuffer& Other)
{
	CleanUp();
	return *this;
}

/**
* Copy Constructor
*/
FSkeletalMeshVertexClothBuffer::FSkeletalMeshVertexClothBuffer(const FSkeletalMeshVertexClothBuffer& Other)
	: VertexData(nullptr),
	Data(nullptr),
	Stride(0),
	NumVertices(0)
{

}

/**
* @return text description for the resource type
*/
FString FSkeletalMeshVertexClothBuffer::GetFriendlyName() const
{
	return TEXT("Skeletal-mesh vertex APEX cloth mesh-mesh mapping buffer");
}

/**
* Delete existing resources
*/
void FSkeletalMeshVertexClothBuffer::CleanUp()
{
	delete VertexData;
	VertexData = nullptr;
}

void FSkeletalMeshVertexClothBuffer::ClearMetaData()
{
	ClothIndexMapping.Empty();
	NumVertices = 0;
}

FBufferRHIRef FSkeletalMeshVertexClothBuffer::CreateRHIBuffer(FRHICommandListBase& RHICmdList)
{
	return FRenderResource::CreateRHIBuffer(RHICmdList, VertexData, NumVertices, BUF_Static | BUF_ShaderResource, TEXT("FSkeletalMeshVertexClothBuffer"));
}

void FSkeletalMeshVertexClothBuffer::InitRHIForStreaming(FRHIBuffer* IntermediateBuffer, FRHIResourceReplaceBatcher& Batcher)
{
	if (VertexBufferRHI && IntermediateBuffer)
	{
		Batcher.EnqueueReplace(VertexBufferRHI, IntermediateBuffer);
	}
}

void FSkeletalMeshVertexClothBuffer::ReleaseRHIForStreaming(FRHIResourceReplaceBatcher& Batcher)
{
	if (VertexBufferRHI)
	{
		Batcher.EnqueueReplace(VertexBufferRHI, nullptr);
	}
}

void FSkeletalMeshVertexClothBuffer::SetMetadata(const TArray<FClothBufferIndexMapping>& InClothIndexMapping, uint32 InStride, uint32 InNumVertices)
{
	ClothIndexMapping = InClothIndexMapping;
	Stride = InStride;
	NumVertices = InNumVertices;
}


/**
* Initialize the RHI resource for this vertex buffer
*/
void FSkeletalMeshVertexClothBuffer::InitRHI(FRHICommandListBase& RHICmdList)
{
	SCOPED_LOADTIMER(FSkeletalMeshVertexClothBuffer_InitRHI);

	VertexBufferRHI = CreateRHIBuffer(RHICmdList);
	if (VertexBufferRHI)
	{
		VertexBufferSRV = RHICmdList.CreateShaderResourceView(
			VertexBufferRHI,
			FRHIViewDesc::CreateBufferSRV()
				.SetType(FRHIViewDesc::EBufferType::Typed)
				.SetFormat(PF_A32B32G32R32F));
	}
}

/**
* Release the RHI resource for this vertex buffer
*/
void FSkeletalMeshVertexClothBuffer::ReleaseRHI()
{
	VertexBufferSRV.SafeRelease();
	// call the base class's ReleaseRHI() since we overrode it
	FVertexBuffer::ReleaseRHI();
}

/**
* Serializer for this class
* @param Ar - archive to serialize to
* @param B - data to serialize
*/
FArchive& operator<<(FArchive& Ar, FSkeletalMeshVertexClothBuffer& VertexBuffer)
{
	FStripDataFlags StripFlags(Ar, 0, FPackageFileVersion::CreateUE4Version(VER_UE4_STATIC_SKELETAL_MESH_SERIALIZATION_FIX));

	if (Ar.IsLoading())
	{
		VertexBuffer.AllocateData();
	}

	if (!StripFlags.IsAudioVisualDataStripped() || Ar.IsCountingMemory())
	{
		if (VertexBuffer.VertexData != NULL)
		{
			VertexBuffer.VertexData->Serialize(Ar);

			// update cached buffer info
			VertexBuffer.NumVertices = VertexBuffer.VertexData->GetNumVertices();
			VertexBuffer.Data = (VertexBuffer.NumVertices > 0) ? VertexBuffer.VertexData->GetDataPointer() : nullptr;
			VertexBuffer.Stride = VertexBuffer.VertexData->GetStride();
		}

		Ar << VertexBuffer.ClothIndexMapping;
	}

	return Ar;
}

void FSkeletalMeshVertexClothBuffer::SerializeMetaData(FArchive& Ar)
{
	Ar << ClothIndexMapping << Stride << NumVertices;
}


/**
* Initializes the buffer with the given vertices.
* @param InVertices - The vertices to initialize the buffer with.
*/
void FSkeletalMeshVertexClothBuffer::Init(const TArrayView<const FMeshToMeshVertData>& InMappingData, const TArrayView<const FClothBufferIndexMapping>& InClothIndexMapping)
{
	// Allocate new data
	AllocateData();

	// Resize the buffer to hold enough data for all passed in vertices
	VertexData->ResizeBuffer(InMappingData.Num());

	Data = VertexData->GetDataPointer();
	Stride = VertexData->GetStride();
	NumVertices = VertexData->GetNumVertices();

	// Copy the vertices into the buffer.
	checkSlow(Stride*NumVertices == sizeof(FMeshToMeshVertData) * InMappingData.Num());
	//appMemcpy(Data, &InMappingData(0), Stride*NumVertices);
	for (int32 Index = 0; Index < InMappingData.Num(); Index++)
	{
		const FMeshToMeshVertData& SourceMapping = InMappingData[Index];
		const int32 DestVertexIndex = Index;
		MappingData(DestVertexIndex) = SourceMapping;
	}
	ClothIndexMapping = InClothIndexMapping;
}

/**
* Allocates the vertex data storage type.
*/
void FSkeletalMeshVertexClothBuffer::AllocateData()
{
	CleanUp();

	VertexData = new TSkeletalMeshVertexData<FMeshToMeshVertData>(true);
}
