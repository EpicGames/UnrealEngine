// Copyright Epic Games, Inc. All Rights Reserved.

#include "Rendering/SkinWeightVertexBuffer.h"
#include "EngineLogs.h"
#include "EngineUtils.h"
#include "Rendering/SkeletalMeshLODModel.h"
#include "ProfilingDebugging/LoadTimeTracker.h"
#include "RHIResourceReplace.h"
#include "SkeletalMeshLegacyCustomVersions.h"
#include "UObject/UE5MainStreamObjectVersion.h"
#include "Rendering/RenderCommandPipes.h"

/*-----------------------------------------------------------------------------
FSkinWeightLookupVertexBuffer
-----------------------------------------------------------------------------*/

FSkinWeightLookupVertexBuffer::FSkinWeightLookupVertexBuffer()
	: bNeedsCPUAccess(false)
	, LookupData(nullptr)
	, Data(nullptr)
	, NumVertices(0)
{
}

FSkinWeightLookupVertexBuffer::FSkinWeightLookupVertexBuffer(const FSkinWeightLookupVertexBuffer &Other)
	: bNeedsCPUAccess(Other.bNeedsCPUAccess)
	, LookupData(nullptr)
	, Data(nullptr)
	, NumVertices(0)
{

}

FSkinWeightLookupVertexBuffer::~FSkinWeightLookupVertexBuffer()
{
	CleanUp();
}

FSkinWeightLookupVertexBuffer& FSkinWeightLookupVertexBuffer::operator=(const FSkinWeightLookupVertexBuffer& Other)
{
	CleanUp();
	bNeedsCPUAccess = Other.bNeedsCPUAccess;
	return *this;
}

void FSkinWeightLookupVertexBuffer::CleanUp()
{
	if (LookupData)
	{
		delete LookupData;
		LookupData = NULL;
	}
}

bool FSkinWeightLookupVertexBuffer::IsLookupDataValid() const
{
	return LookupData != NULL;
}

void FSkinWeightLookupVertexBuffer::Init(uint32 InNumVertices)
{
	AllocateData();
	ResizeBuffer(InNumVertices);
}

void FSkinWeightLookupVertexBuffer::ResizeBuffer(uint32 InNumVertices)
{
	NumVertices = InNumVertices;
	LookupData->ResizeBuffer(NumVertices);
	Data = NumVertices ? LookupData->GetDataPointer() : nullptr;
}

FArchive& operator<<(FArchive& Ar, FSkinWeightLookupVertexBuffer& VertexBuffer)
{
	FStripDataFlags StripFlags(Ar);

	Ar.UsingCustomVersion(FAnimObjectVersion::GUID);
	VertexBuffer.SerializeMetaData(Ar);

	if (Ar.IsLoading() || VertexBuffer.LookupData == NULL)
	{
		// If we're loading, or we have no valid buffer, allocate container.
		VertexBuffer.AllocateData();
	}

	// if Ar is counting, it still should serialize. Need to count VertexData
	if (!StripFlags.IsAudioVisualDataStripped() || Ar.IsCountingMemory())
	{
		if (VertexBuffer.LookupData != NULL)
		{
			if (Ar.CustomVer(FAnimObjectVersion::GUID) >= FAnimObjectVersion::UnlimitedBoneInfluences)
			{
				VertexBuffer.LookupData->Serialize(Ar);
			}

			if (!Ar.IsCountingMemory())
			{
				// update cached buffer info
				VertexBuffer.Data = (VertexBuffer.NumVertices > 0 && VertexBuffer.LookupData->GetResourceArray()->GetResourceDataSize()) ? VertexBuffer.LookupData->GetDataPointer() : nullptr;
			}
		}
	}

	return Ar;
}

void FSkinWeightLookupVertexBuffer::SerializeMetaData(FArchive& Ar)
{
	Ar.UsingCustomVersion(FAnimObjectVersion::GUID);
	if (Ar.CustomVer(FAnimObjectVersion::GUID) >= FAnimObjectVersion::UnlimitedBoneInfluences)
	{
		Ar << NumVertices;
	}
}

void FSkinWeightLookupVertexBuffer::SetMetaData(uint32 InNumVertices)
{
	NumVertices = InNumVertices;
}

void FSkinWeightLookupVertexBuffer::CopyMetaData(const FSkinWeightLookupVertexBuffer& Other)
{
	NumVertices = Other.NumVertices;
}

void FSkinWeightLookupVertexBuffer::AllocateData()
{
	// Clear any old WeightData before allocating.
	CleanUp();

	LookupData = new TStaticMeshVertexData<uint32>(bNeedsCPUAccess);
}

void FSkinWeightLookupVertexBuffer::InitRHI(FRHICommandListBase& RHICmdList)
{
	// BUF_ShaderResource is needed for support of the SkinCache (we could make is dependent on GEnableGPUSkinCacheShaders or are there other users?)
	VertexBufferRHI = CreateRHIBuffer(RHICmdList);
	if (VertexBufferRHI)
	{
		bool bSRV = GPixelFormats[PixelFormat].Supported;
		// When bAllowCPUAccess is true, the meshes is likely going to be used for Niagara to spawn particles on mesh surface.
		// And it can be the case for CPU *and* GPU access: no differenciation today. That is why we create a SRV in this case.
		// This also avoid setting lots of states on all the members of all the different buffers used by meshes. Follow up: https://jira.it.epicgames.net/browse/UE-69376.
		bSRV |= GetNeedsCPUAccess();

		if (bSRV)
		{
			SRVValue = RHICmdList.CreateShaderResourceView(
				VertexBufferRHI,
				FRHIViewDesc::CreateBufferSRV()
					.SetType(FRHIViewDesc::EBufferType::Typed)
					.SetFormat(PixelFormat));
		}
	}
}

void FSkinWeightLookupVertexBuffer::ReleaseRHI()
{
	SRVValue.SafeRelease();

	FVertexBuffer::ReleaseRHI();
}

void FSkinWeightLookupVertexBuffer::InitRHIForStreaming(FRHIBuffer* IntermediateBuffer, FRHIResourceReplaceBatcher& Batcher)
{
	if (VertexBufferRHI && IntermediateBuffer)
	{
		Batcher.EnqueueReplace(VertexBufferRHI, IntermediateBuffer);
	}
}

void FSkinWeightLookupVertexBuffer::ReleaseRHIForStreaming(FRHIResourceReplaceBatcher& Batcher)
{
	if (VertexBufferRHI)
	{
		Batcher.EnqueueReplace(VertexBufferRHI, nullptr);
	}
}

void FSkinWeightLookupVertexBuffer::GetWeightOffsetAndInfluenceCount(uint32 VertexIndex, uint32& OutWeightOffset, uint32& OutInfluenceCount) const
{
	uint32 Offset = VertexIndex * 4;
	uint32 DataUInt32 = *((uint32*)(&Data[Offset]));
	OutWeightOffset = DataUInt32 >> 8;
	OutInfluenceCount = DataUInt32 & 0xff;
}

void FSkinWeightLookupVertexBuffer::SetWeightOffsetAndInfluenceCount(uint32 VertexIndex, uint32 WeightOffset, uint32 InfluenceCount)
{
	uint32 Offset = VertexIndex * 4;
	uint32* DataUInt32 = (uint32*)(&Data[Offset]);
	*DataUInt32 = (WeightOffset << 8) | InfluenceCount;
}

FBufferRHIRef FSkinWeightLookupVertexBuffer::CreateRHIBuffer(FRHICommandListBase& RHICmdList)
{
	return FRenderResource::CreateRHIBuffer(RHICmdList, LookupData, NumVertices, BUF_Static | BUF_ShaderResource | BUF_SourceCopy, TEXT("FSkinWeightLookupVertexBuffer"));
}

/*-----------------------------------------------------------------------------
FSkinWeightDataVertexBuffer
-----------------------------------------------------------------------------*/

FSkinWeightDataVertexBuffer::FSkinWeightDataVertexBuffer()
:	bNeedsCPUAccess(false)
,	bVariableBonesPerVertex(false)
,	MaxBoneInfluences(MAX_INFLUENCES_PER_STREAM)
,	bUse16BitBoneIndex(false)
,	bUse16BitBoneWeight(false)
,	WeightData(nullptr)
,	Data(nullptr)
,	NumVertices(0)
,	NumBoneWeights(0)
{
}

FSkinWeightDataVertexBuffer::FSkinWeightDataVertexBuffer( const FSkinWeightDataVertexBuffer &Other )
	: bNeedsCPUAccess(Other.bNeedsCPUAccess)
	, bVariableBonesPerVertex(Other.bVariableBonesPerVertex)
	, MaxBoneInfluences(Other.MaxBoneInfluences)
	, bUse16BitBoneIndex(Other.bUse16BitBoneIndex)
	, bUse16BitBoneWeight(Other.bUse16BitBoneWeight)
	, WeightData(nullptr)
	, Data(nullptr)
	, NumVertices(0)
	, NumBoneWeights(0)
{
	
}

FSkinWeightDataVertexBuffer::~FSkinWeightDataVertexBuffer()
{
	CleanUp();
}

FSkinWeightDataVertexBuffer& FSkinWeightDataVertexBuffer::operator=(const FSkinWeightDataVertexBuffer& Other)
{
	CleanUp();
	bNeedsCPUAccess = Other.bNeedsCPUAccess;
	bVariableBonesPerVertex = Other.bVariableBonesPerVertex;
	MaxBoneInfluences = Other.MaxBoneInfluences;
	bUse16BitBoneIndex = Other.bUse16BitBoneIndex;
	bUse16BitBoneWeight = Other.bUse16BitBoneWeight;
	return *this;
}

void FSkinWeightDataVertexBuffer::CleanUp()
{
	if (WeightData)
	{
		delete WeightData;
		WeightData = nullptr;
	}
}

void FSkinWeightDataVertexBuffer::Init(uint32 InNumWeights, uint32 InNumVertices)
{
	AllocateData();
	ResizeBuffer(InNumWeights, InNumVertices);
}

void FSkinWeightDataVertexBuffer::ResizeBuffer(uint32 InNumWeights, uint32 InNumVertices)
{
	NumBoneWeights = InNumWeights;
	NumVertices = InNumVertices;
	WeightData->ResizeBuffer(GetVertexDataSize());

	if (NumBoneWeights > 0)
	{
		Data = WeightData->GetDataPointer();
	}
}

FArchive& operator<<(FArchive& Ar, FSkinWeightDataVertexBuffer& VertexBuffer)
{
	FStripDataFlags StripFlags(Ar);

	Ar.UsingCustomVersion(FAnimObjectVersion::GUID);
	VertexBuffer.SerializeMetaData(Ar);

	if (Ar.IsLoading() || VertexBuffer.WeightData == nullptr)
	{
		// If we're loading, or we have no valid buffer, allocate container.
		VertexBuffer.AllocateData();
	}

	// if Ar is counting, it still should serialize. Need to count VertexData
	if (!StripFlags.IsAudioVisualDataStripped() || Ar.IsCountingMemory())
	{
		if (VertexBuffer.WeightData != NULL)
		{
			if (Ar.IsLoading() && Ar.CustomVer(FAnimObjectVersion::GUID) < FAnimObjectVersion::UnlimitedBoneInfluences)
			{
				FStaticMeshVertexDataInterface* LegacyWeightData = nullptr;
				bool bExtraBoneInfluences = VertexBuffer.MaxBoneInfluences > MAX_INFLUENCES_PER_STREAM;
				if (bExtraBoneInfluences)
				{
					if (VertexBuffer.bUse16BitBoneIndex)
					{
						LegacyWeightData = new FSkinWeightVertexData< TLegacySkinWeightInfo<true, FBoneIndex16> >(VertexBuffer.bNeedsCPUAccess);
					}
					else
					{
						LegacyWeightData = new FSkinWeightVertexData< TLegacySkinWeightInfo<true, FBoneIndex8> >(VertexBuffer.bNeedsCPUAccess);
					}
				}
				else
				{
					if (VertexBuffer.bUse16BitBoneIndex)
					{
						LegacyWeightData = new FSkinWeightVertexData< TLegacySkinWeightInfo<false, FBoneIndex16> >(VertexBuffer.bNeedsCPUAccess);
					}
					else
					{
						LegacyWeightData = new FSkinWeightVertexData< TLegacySkinWeightInfo<false, FBoneIndex8> >(VertexBuffer.bNeedsCPUAccess);
					}
				}
				bool bForcePerElementSerialization = Ar.IsLoading() && Ar.CustomVer(FAnimObjectVersion::GUID) < FAnimObjectVersion::IncreaseBoneIndexLimitPerChunk;
				LegacyWeightData->Serialize(Ar, bForcePerElementSerialization);
	
				uint32 BufferSize = LegacyWeightData->GetResourceArray()->GetResourceDataSize();
				check(VertexBuffer.GetVertexDataSize() == BufferSize);
				VertexBuffer.WeightData->ResizeBuffer(BufferSize);
				FMemory::Memcpy(VertexBuffer.WeightData->GetDataPointer(), LegacyWeightData->GetDataPointer(), BufferSize);
				VertexBuffer.NumVertices = LegacyWeightData->Num();

				if (LegacyWeightData)
				{
					delete LegacyWeightData;
				}
			}
			else
			{
				VertexBuffer.WeightData->Serialize(Ar);
			}

			if (!Ar.IsCountingMemory())
			{
				// update cached buffer info
				VertexBuffer.Data = (VertexBuffer.NumBoneWeights > 0 && VertexBuffer.WeightData->GetResourceArray()->GetResourceDataSize()) ? VertexBuffer.WeightData->GetDataPointer() : nullptr;
			}
		}
	}

	return Ar;
}

void FSkinWeightDataVertexBuffer::SerializeMetaData(FArchive& Ar)
{
	Ar.UsingCustomVersion(FSkeletalMeshCustomVersion::GUID);
	Ar.UsingCustomVersion(FAnimObjectVersion::GUID);
	Ar.UsingCustomVersion(FUE5MainStreamObjectVersion::GUID);

	if (Ar.IsLoading() && Ar.CustomVer(FAnimObjectVersion::GUID) < FAnimObjectVersion::UnlimitedBoneInfluences)
	{
		bool bExtraBoneInfluences = false;
		if (Ar.CustomVer(FSkeletalMeshCustomVersion::GUID) < FSkeletalMeshCustomVersion::SplitModelAndRenderData)
		{
			Ar << bExtraBoneInfluences << NumVertices;
		}
		else
		{		
			uint32 Stride = 0;
			Ar << bExtraBoneInfluences << Stride << NumVertices;
		}
		MaxBoneInfluences = bExtraBoneInfluences ? EXTRA_BONE_INFLUENCES : MAX_INFLUENCES_PER_STREAM;
		NumBoneWeights = MaxBoneInfluences * NumVertices;
		bVariableBonesPerVertex = false;
	}
	else
	{
		Ar << bVariableBonesPerVertex << MaxBoneInfluences << NumBoneWeights << NumVertices;
	}
	
	// bUse16BitBoneIndex doesn't exist before version IncreaseBoneIndexLimitPerChunk
	if (Ar.CustomVer(FAnimObjectVersion::GUID) >= FAnimObjectVersion::IncreaseBoneIndexLimitPerChunk)
	{
		Ar << bUse16BitBoneIndex;
	}
	if (Ar.CustomVer(FUE5MainStreamObjectVersion::GUID) >= FUE5MainStreamObjectVersion::IncreasedSkinWeightPrecision)
	{
		Ar << bUse16BitBoneWeight;
	}
}

void FSkinWeightDataVertexBuffer::SetMetaData(uint32 InNumVertices, uint32 InMaxBoneInfluences, bool bInUse16BitBoneIndex, bool bInUse16BitBoneWeight)
{
	NumVertices = InNumVertices;
	bUse16BitBoneIndex = bInUse16BitBoneIndex;
	bUse16BitBoneWeight = bInUse16BitBoneWeight;

	SetMaxBoneInfluences(InMaxBoneInfluences);
	NumBoneWeights = NumVertices * MaxBoneInfluences;
}

void FSkinWeightDataVertexBuffer::CopyMetaData(const FSkinWeightDataVertexBuffer& Other)
{
	bVariableBonesPerVertex = Other.bVariableBonesPerVertex;
	MaxBoneInfluences = Other.MaxBoneInfluences;
	bUse16BitBoneIndex = Other.bUse16BitBoneIndex;
	bUse16BitBoneWeight = Other.bUse16BitBoneWeight;
	NumBoneWeights = Other.NumBoneWeights;
}

FBufferRHIRef FSkinWeightDataVertexBuffer::CreateRHIBuffer(FRHICommandListBase& RHICmdList)
{
	// BUF_ShaderResource is needed for support of the SkinCache (we could make is dependent on GEnableGPUSkinCacheShaders or are there other users?)
	return FRenderResource::CreateRHIBuffer(RHICmdList, WeightData, NumBoneWeights, BUF_Static | BUF_ShaderResource | BUF_SourceCopy, TEXT("FSkinWeightDataVertexBuffer"));
}

void FSkinWeightDataVertexBuffer::InitRHIForStreaming(FRHIBuffer* IntermediateBuffer, FRHIResourceReplaceBatcher& Batcher)
{
	if (VertexBufferRHI && IntermediateBuffer)
	{
		Batcher.EnqueueReplace(VertexBufferRHI, IntermediateBuffer);
	}
}

void FSkinWeightDataVertexBuffer::ReleaseRHIForStreaming(FRHIResourceReplaceBatcher& Batcher)
{
	if (VertexBufferRHI)
	{
		Batcher.EnqueueReplace(VertexBufferRHI, nullptr);
	}
}

bool FSkinWeightDataVertexBuffer::IsWeightDataValid() const
{
	return WeightData != nullptr;
}

void FSkinWeightDataVertexBuffer::InitRHI(FRHICommandListBase& RHICmdList)
{
	SCOPED_LOADTIMER(FSkinWeightVertexBuffer_InitRHI);

	// BUF_ShaderResource is needed for support of the SkinCache (we could make is dependent on GEnableGPUSkinCacheShaders or are there other users?)
	VertexBufferRHI = CreateRHIBuffer(RHICmdList);

	bool bSRV = VertexBufferRHI && GPixelFormats[GetPixelFormat()].Supported;
	// When bAllowCPUAccess is true, the meshes is likely going to be used for Niagara to spawn particles on mesh surface.
	// And it can be the case for CPU *and* GPU access: no differenciation today. That is why we create a SRV in this case.
	// This also avoid setting lots of states on all the members of all the different buffers used by meshes. Follow up: https://jira.it.epicgames.net/browse/UE-69376.
	bSRV |= GetNeedsCPUAccess();

	if (bSRV)
	{
		SRVValue = RHICmdList.CreateShaderResourceView(
			VertexBufferRHI,
			FRHIViewDesc::CreateBufferSRV()
				.SetType(FRHIViewDesc::EBufferType::Typed)
				.SetFormat(GetPixelFormat()));
	}
}

void FSkinWeightDataVertexBuffer::ReleaseRHI()
{
	SRVValue.SafeRelease();

	FVertexBuffer::ReleaseRHI();
}

void FSkinWeightDataVertexBuffer::AllocateData()
{
	// Clear any old WeightData before allocating.
	CleanUp();

	WeightData = new FSkinWeightVertexData<uint8>(bNeedsCPUAccess);
}

void FSkinWeightDataVertexBuffer::SetMaxBoneInfluences(uint32 InMaxBoneInfluences)
{
	check(InMaxBoneInfluences <= MAX_TOTAL_INFLUENCES);
	if (InMaxBoneInfluences <= MAX_INFLUENCES_PER_STREAM)
		MaxBoneInfluences = MAX_INFLUENCES_PER_STREAM;
	else if (InMaxBoneInfluences <= EXTRA_BONE_INFLUENCES)
		MaxBoneInfluences = EXTRA_BONE_INFLUENCES;
	else
		MaxBoneInfluences = MAX_TOTAL_INFLUENCES;

	bVariableBonesPerVertex = FGPUBaseSkinVertexFactory::UseUnlimitedBoneInfluences(MaxBoneInfluences);
	if (MaxBoneInfluences == MAX_TOTAL_INFLUENCES && !bVariableBonesPerVertex)
	{
		MaxBoneInfluences = EXTRA_BONE_INFLUENCES;
		UE_LOGF(LogSkeletalMesh, Error, "Skeletal mesh of %d bone influences requires unlimited bone influence mode. Influence truncation will occur and will not render correctly.", InMaxBoneInfluences);
	}
}

bool FSkinWeightDataVertexBuffer::GetRigidWeightBone(uint32 VertexWeightOffset, uint32 VertexInfluenceCount, int32& OutBoneIndex) const
{
	bool bIsRigid = false;

	uint8* BoneData = Data + VertexWeightOffset;
	uint32 BoneWeightOffset = GetBoneIndexByteSize() * VertexInfluenceCount;
	for (uint32 InfluenceIndex = 0; InfluenceIndex < VertexInfluenceCount; InfluenceIndex++)
	{
		uint8 BoneWeight = BoneData[BoneWeightOffset + InfluenceIndex];
		if (BoneWeight == 255)
		{
			bIsRigid = true;
			if (Use16BitBoneIndex())
			{
				FBoneIndex16* BoneIndex16Ptr = (FBoneIndex16*)BoneData;
				OutBoneIndex = BoneIndex16Ptr[InfluenceIndex];
			}
			else
			{
				OutBoneIndex = BoneData[InfluenceIndex];
			}
			break;
		}
	}

	return bIsRigid;
}

uint32 FSkinWeightDataVertexBuffer::GetBoneIndex(uint32 VertexWeightOffset, uint32 VertexInfluenceCount, uint32 InfluenceIndex) const
{	
	if (InfluenceIndex < VertexInfluenceCount)
	{
		uint8* BoneData = Data + VertexWeightOffset;
		if (Use16BitBoneIndex())
		{
			FBoneIndex16* BoneIndex16Ptr = (FBoneIndex16*)BoneData;
			return BoneIndex16Ptr[InfluenceIndex];
		}
		else
		{
			return BoneData[InfluenceIndex];
		}
	}
	else
	{
		return 0;
	}
}

void FSkinWeightDataVertexBuffer::SetBoneIndex(uint32 VertexWeightOffset, uint32 VertexInfluenceCount, uint32 InfluenceIndex, uint32 BoneIndex)
{
	if (InfluenceIndex < VertexInfluenceCount)
	{
		uint8* BoneData = Data + VertexWeightOffset;
		if (Use16BitBoneIndex())
		{
			FBoneIndex16* BoneIndex16Ptr = (FBoneIndex16*)BoneData;
			BoneIndex16Ptr[InfluenceIndex] = BoneIndex;
		}
		else
		{
			BoneData[InfluenceIndex] = BoneIndex;
		}
	}
	else
	{
		check(false);
	}
}

uint16 FSkinWeightDataVertexBuffer::GetBoneWeight(uint32 VertexWeightOffset, uint32 VertexInfluenceCount, uint32 InfluenceIndex) const
{
	if (InfluenceIndex < VertexInfluenceCount)
	{
		const uint32 BoneWeightOffset = GetBoneIndexByteSize() * VertexInfluenceCount;
		const uint8* BoneData = Data + VertexWeightOffset + BoneWeightOffset;
		const uint8* BoneVertexData = &BoneData[InfluenceIndex * GetBoneWeightByteSize()];  

		if (Use16BitBoneWeight())
		{
			return *reinterpret_cast<const uint16*>(BoneVertexData);
		}
		else
		{
			return (static_cast<uint16>(*BoneVertexData) << 8) | *BoneVertexData;
		}
	}
	else
	{
		return 0;
	}
}

void FSkinWeightDataVertexBuffer::SetBoneWeight(uint32 VertexWeightOffset, uint32 VertexInfluenceCount, uint32 InfluenceIndex, uint16 BoneWeight)
{
	if (InfluenceIndex < VertexInfluenceCount)
	{
		const uint32 BoneWeightOffset = GetBoneIndexByteSize() * VertexInfluenceCount;
		uint8* BoneData = Data + VertexWeightOffset + BoneWeightOffset;
		uint8* BoneVertexData = &BoneData[InfluenceIndex * GetBoneWeightByteSize()];  

		if (Use16BitBoneWeight())
		{
			*reinterpret_cast<uint16*>(BoneVertexData) = BoneWeight;
		}
		else
		{
			*BoneVertexData = static_cast<uint8>(BoneWeight >> 8);
		}

	}
	else
	{
		check(false);
	}
}

void FSkinWeightDataVertexBuffer::ResetVertexBoneWeights(uint32 VertexWeightOffset, uint32 VertexInfluenceCount)
{
	if (VertexInfluenceCount > 0)
	{
		FMemory::Memzero(Data + VertexWeightOffset, GetBoneIndexAndWeightByteSize() * VertexInfluenceCount);
	}
}

void FSkinWeightDataVertexBuffer::CopyDataFromBuffer(const uint8* InSkinWeightData, uint32 InNumVertices)
{
	Init(InNumVertices * GetMaxBoneInfluences(), InNumVertices);
	FMemory::Memcpy(Data, InSkinWeightData, GetVertexDataSize());
}

/*-----------------------------------------------------------------------------
FSkinWeightVertexBuffer
-----------------------------------------------------------------------------*/

namespace SkeletalMeshResourceNames
{
	static const FName DataVertexBufferName("FSkinWeightVertexBuffer.DataVertexBuffer");
	static const FName LookupVertexBufferName("FSkinWeightVertexBuffer.LookupVertexBuffer");
} // SkeletalMeshResourceNames

FSkinWeightVertexBuffer::FSkinWeightVertexBuffer()
{
	DataVertexBuffer.SetResourceName(SkeletalMeshResourceNames::DataVertexBufferName);
	LookupVertexBuffer.SetResourceName(SkeletalMeshResourceNames::LookupVertexBufferName);
}

FSkinWeightVertexBuffer::FSkinWeightVertexBuffer(const FSkinWeightVertexBuffer& Other)
{
	bool bNeedsCPUAccess = Other.GetNeedsCPUAccess();
	DataVertexBuffer.SetNeedsCPUAccess(bNeedsCPUAccess);
	LookupVertexBuffer.SetNeedsCPUAccess(bNeedsCPUAccess);

	SetMaxBoneInfluences(Other.GetMaxBoneInfluences());
	SetUse16BitBoneIndex(Other.Use16BitBoneIndex());
	SetUse16BitBoneWeight(Other.Use16BitBoneWeight());
}

FSkinWeightVertexBuffer::~FSkinWeightVertexBuffer()
{
	CleanUp();
}

FSkinWeightVertexBuffer& FSkinWeightVertexBuffer::operator=(const FSkinWeightVertexBuffer& Other)
{
	CleanUp();

	bool bNeedsCPUAccess = Other.GetNeedsCPUAccess();
	DataVertexBuffer.SetNeedsCPUAccess(bNeedsCPUAccess);
	LookupVertexBuffer.SetNeedsCPUAccess(bNeedsCPUAccess);

	SetMaxBoneInfluences(Other.GetMaxBoneInfluences());
	SetUse16BitBoneIndex(Other.Use16BitBoneIndex());
	SetUse16BitBoneWeight(Other.Use16BitBoneWeight());

	return *this;
}

void FSkinWeightVertexBuffer::CleanUp()
{
	DataVertexBuffer.CleanUp();
	LookupVertexBuffer.CleanUp();
}

void FSkinWeightVertexBuffer::RebuildLookupVertexBuffer()
{
	uint32 MaxBoneInfluences = DataVertexBuffer.GetMaxBoneInfluences();
	uint32 NumVertices = DataVertexBuffer.GetNumBoneWeights() / DataVertexBuffer.GetMaxBoneInfluences();
	LookupVertexBuffer.Init(NumVertices);

	uint32 WeightOffset = 0;
	for (uint32 VertIdx = 0; VertIdx < NumVertices; VertIdx++)
	{
		LookupVertexBuffer.SetWeightOffsetAndInfluenceCount(VertIdx, WeightOffset * GetBoneIndexAndWeightByteSize(), MaxBoneInfluences);
		WeightOffset += MaxBoneInfluences;
	}
}

FArchive& operator<<(FArchive& Ar, FSkinWeightVertexBuffer& VertexBuffer)
{
	Ar.UsingCustomVersion(FAnimObjectVersion::GUID);	
	Ar << VertexBuffer.DataVertexBuffer;

	if (Ar.IsLoading() && Ar.CustomVer(FAnimObjectVersion::GUID) < FAnimObjectVersion::UnlimitedBoneInfluences)
	{
		// LookupVertexBuffer doesn't exist before this version, so construct its content from scratch
		VertexBuffer.RebuildLookupVertexBuffer();
	}
	else
	{
		Ar << VertexBuffer.LookupVertexBuffer;
	}

	return Ar;
}

void FSkinWeightVertexBuffer::SerializeMetaData(FArchive& Ar)
{
	DataVertexBuffer.SerializeMetaData(Ar);
	LookupVertexBuffer.SerializeMetaData(Ar);
}

void FSkinWeightVertexBuffer::CopyMetaData(const FSkinWeightVertexBuffer& Other)
{
	DataVertexBuffer.CopyMetaData(Other.DataVertexBuffer);
	LookupVertexBuffer.CopyMetaData(Other.LookupVertexBuffer);
}

FSkinWeightRHIInfo FSkinWeightVertexBuffer::CreateRHIBuffer(FRHICommandListBase& RHICmdList)
{
	FSkinWeightRHIInfo RHIInfo;
	RHIInfo.DataVertexBufferRHI = DataVertexBuffer.CreateRHIBuffer(RHICmdList);
	RHIInfo.LookupVertexBufferRHI = LookupVertexBuffer.CreateRHIBuffer(RHICmdList);
	return RHIInfo;
}

GPUSkinBoneInfluenceType FSkinWeightVertexBuffer::GetBoneInfluenceType() const
{
	if (GetVariableBonesPerVertex())
	{
		return GPUSkinBoneInfluenceType::UnlimitedBoneInfluence;
	}
	else
	{
		return GPUSkinBoneInfluenceType::DefaultBoneInfluence;
	}
}

void FSkinWeightVertexBuffer::GetVertexInfluenceOffsetCount(uint32 VertexIndex, uint32& VertexWeightOffset, uint32& VertexInfluenceCount) const
{
	if (GetVariableBonesPerVertex())
	{
		LookupVertexBuffer.GetWeightOffsetAndInfluenceCount(VertexIndex, VertexWeightOffset, VertexInfluenceCount);
	}
	else
	{
		VertexWeightOffset = GetConstantInfluencesVertexStride() * VertexIndex;
		VertexInfluenceCount = GetMaxBoneInfluences();
	}
}

bool FSkinWeightVertexBuffer::GetRigidWeightBone(uint32 VertexIndex, int32& OutBoneIndex) const
{
	uint32 VertexWeightOffset = 0;
	uint32 VertexInfluenceCount = 0;
	GetVertexInfluenceOffsetCount(VertexIndex, VertexWeightOffset, VertexInfluenceCount);
	return DataVertexBuffer.GetRigidWeightBone(VertexWeightOffset, VertexInfluenceCount, OutBoneIndex);
}

uint32 FSkinWeightVertexBuffer::GetBoneIndex(uint32 VertexIndex, uint32 InfluenceIndex) const
{
	uint32 VertexWeightOffset = 0;
	uint32 VertexInfluenceCount = 0;
	GetVertexInfluenceOffsetCount(VertexIndex, VertexWeightOffset, VertexInfluenceCount);
	return DataVertexBuffer.GetBoneIndex(VertexWeightOffset, VertexInfluenceCount, InfluenceIndex);
}

void FSkinWeightVertexBuffer::SetBoneIndex(uint32 VertexIndex, uint32 InfluenceIndex, uint32 BoneIndex)
{
	uint32 VertexWeightOffset = 0;
	uint32 VertexInfluenceCount = 0;
	GetVertexInfluenceOffsetCount(VertexIndex, VertexWeightOffset, VertexInfluenceCount);
	DataVertexBuffer.SetBoneIndex(VertexWeightOffset, VertexInfluenceCount, InfluenceIndex, BoneIndex);
}

uint16 FSkinWeightVertexBuffer::GetBoneWeight(uint32 VertexIndex, uint32 InfluenceIndex) const
{
	uint32 VertexWeightOffset = 0;
	uint32 VertexInfluenceCount = 0;
	GetVertexInfluenceOffsetCount(VertexIndex, VertexWeightOffset, VertexInfluenceCount);
	return DataVertexBuffer.GetBoneWeight(VertexWeightOffset, VertexInfluenceCount, InfluenceIndex);
}

void FSkinWeightVertexBuffer::SetBoneWeight(uint32 VertexIndex, uint32 InfluenceIndex, uint16 BoneWeight)
{
	uint32 VertexWeightOffset = 0;
	uint32 VertexInfluenceCount = 0;
	GetVertexInfluenceOffsetCount(VertexIndex, VertexWeightOffset, VertexInfluenceCount);
	DataVertexBuffer.SetBoneWeight(VertexWeightOffset, VertexInfluenceCount, InfluenceIndex, BoneWeight);
}

void FSkinWeightVertexBuffer::ResetVertexBoneWeights(uint32 VertexIndex)
{
	uint32 VertexWeightOffset = 0;
	uint32 VertexInfluenceCount = 0;
	GetVertexInfluenceOffsetCount(VertexIndex, VertexWeightOffset, VertexInfluenceCount);
	DataVertexBuffer.ResetVertexBoneWeights(VertexWeightOffset, VertexInfluenceCount);
}

void FSkinWeightVertexBuffer::SetOwnerName(const FName& OwnerName)
{
	LookupVertexBuffer.SetOwnerName(OwnerName);
	DataVertexBuffer.SetOwnerName(OwnerName);
}

void FSkinWeightVertexBuffer::BeginInitResources()
{
	BeginInitResource(&LookupVertexBuffer, &UE::RenderCommandPipe::SkeletalMesh);
	BeginInitResource(&DataVertexBuffer, &UE::RenderCommandPipe::SkeletalMesh);
}

void FSkinWeightVertexBuffer::BeginReleaseResources()
{
	BeginReleaseResource(&LookupVertexBuffer, &UE::RenderCommandPipe::SkeletalMesh);
	BeginReleaseResource(&DataVertexBuffer, &UE::RenderCommandPipe::SkeletalMesh);
}

void FSkinWeightVertexBuffer::ReleaseResources()
{
	check(IsInParallelRenderingThread());
	LookupVertexBuffer.ReleaseResource();
	DataVertexBuffer.ReleaseResource();
}

#if WITH_EDITOR
void FSkinWeightVertexBuffer::Init(const TArray<FSoftSkinVertex>& InVertices)
{
	static_assert(sizeof(FSoftSkinVertex::InfluenceBones) == sizeof(FSkinWeightInfo::InfluenceBones));
	static_assert(sizeof(FSoftSkinVertex::InfluenceWeights) == sizeof(FSkinWeightInfo::InfluenceWeights));
	
	TArray<FSkinWeightInfo> VertexWeightInfos;
	VertexWeightInfos.AddUninitialized(InVertices.Num());

	for (int32 VertIdx = 0; VertIdx < InVertices.Num(); VertIdx++)
	{
		const FSoftSkinVertex& SrcVertex = InVertices[VertIdx];
		FSkinWeightInfo& DstVertex = VertexWeightInfos[VertIdx];
		FMemory::Memcpy(DstVertex.InfluenceBones, SrcVertex.InfluenceBones, MAX_TOTAL_INFLUENCES * sizeof(FBoneIndexType));
		FMemory::Memcpy(DstVertex.InfluenceWeights, SrcVertex.InfluenceWeights, MAX_TOTAL_INFLUENCES * sizeof(uint16));

	}

	*this = VertexWeightInfos;
}
#endif //WITH_EDITOR

FSkinWeightVertexBuffer& FSkinWeightVertexBuffer::operator=(const TArray<FSkinWeightInfo>& InVertices)
{
	uint32 TotalNumUsedBones = 0;
	bool bVariableBonesPerVertex = GetVariableBonesPerVertex();

	if (bVariableBonesPerVertex)
	{
		LookupVertexBuffer.Init(InVertices.Num());
		for (int32 VertIdx = 0; VertIdx < InVertices.Num(); VertIdx++)
		{
			int32 NumUsedBones = 0;
			for (int32 InfluenceIdx = 0; InfluenceIdx < MAX_TOTAL_INFLUENCES; InfluenceIdx++)
			{
				if (InVertices[VertIdx].InfluenceWeights[InfluenceIdx] > 0)
				{
					NumUsedBones++;
				}
			}

			LookupVertexBuffer.SetWeightOffsetAndInfluenceCount(VertIdx, TotalNumUsedBones * GetBoneIndexAndWeightByteSize(), NumUsedBones);
			TotalNumUsedBones += NumUsedBones;
		}
	}
	else
	{
		TotalNumUsedBones = GetMaxBoneInfluences() * InVertices.Num();
	}

	DataVertexBuffer.Init(TotalNumUsedBones, InVertices.Num());
	for (int32 VertIdx = 0; VertIdx < InVertices.Num(); VertIdx++)
	{
		if (bVariableBonesPerVertex)
		{
			// Copy the weights and sort in descending order
			FSkinWeightInfo VertexWeights = InVertices[VertIdx];
			for (uint32 i = 0; i < MAX_TOTAL_INFLUENCES; i++)
			{
				uint32 MaxWeightIdx = i;
				for (uint32 j = i+1; j < MAX_TOTAL_INFLUENCES; j++)
				{
					if (VertexWeights.InfluenceWeights[j] > VertexWeights.InfluenceWeights[MaxWeightIdx])
					{
						MaxWeightIdx = j;
					}
				}
				if (MaxWeightIdx != i)
				{
					Exchange(VertexWeights.InfluenceBones[i], VertexWeights.InfluenceBones[MaxWeightIdx]);
					Exchange(VertexWeights.InfluenceWeights[i], VertexWeights.InfluenceWeights[MaxWeightIdx]);
				}
			}

			int32 NumUsedBones = 0;
			for (uint32 InfluenceIdx = 0; InfluenceIdx < GetMaxBoneInfluences(); InfluenceIdx++)
			{
				if (VertexWeights.InfluenceWeights[InfluenceIdx] > 0)
				{
					SetBoneIndex(VertIdx, NumUsedBones, VertexWeights.InfluenceBones[InfluenceIdx]);
					SetBoneWeight(VertIdx, NumUsedBones, VertexWeights.InfluenceWeights[InfluenceIdx]);
					NumUsedBones++;
				}
			}
		}
		else
		{
			const FSkinWeightInfo& VertexWeights = InVertices[VertIdx];
			for (uint32 InfluenceIdx = 0; InfluenceIdx < GetMaxBoneInfluences(); InfluenceIdx++)
			{
				SetBoneIndex(VertIdx, InfluenceIdx, VertexWeights.InfluenceBones[InfluenceIdx]);
				SetBoneWeight(VertIdx, InfluenceIdx, VertexWeights.InfluenceWeights[InfluenceIdx]);
			}
		}
	}

	return *this;
}

void FSkinWeightVertexBuffer::GetSkinWeights(TArray<FSkinWeightInfo>& OutVertices) const
{
	OutVertices.SetNum(GetNumVertices());
	for (uint32 VertIdx = 0; VertIdx < GetNumVertices(); VertIdx++)
	{
		for (int32 InfluenceIdx = 0; InfluenceIdx < MAX_TOTAL_INFLUENCES; InfluenceIdx++)
		{
			OutVertices[VertIdx].InfluenceBones[InfluenceIdx] = GetBoneIndex(VertIdx, InfluenceIdx);
			OutVertices[VertIdx].InfluenceWeights[InfluenceIdx] = GetBoneWeight(VertIdx, InfluenceIdx);
		}
	}
}

FSkinWeightInfo FSkinWeightVertexBuffer::GetVertexSkinWeights(uint32 VertexIndex) const
{
	FSkinWeightInfo OutVertex;
	for (int32 InfluenceIdx = 0; InfluenceIdx < MAX_TOTAL_INFLUENCES; InfluenceIdx++)
	{
		OutVertex.InfluenceBones[InfluenceIdx] = GetBoneIndex(VertexIndex, InfluenceIdx);
		OutVertex.InfluenceWeights[InfluenceIdx] = GetBoneWeight(VertexIndex, InfluenceIdx);
	}
	return OutVertex;
}
void FSkinWeightVertexBuffer::CopySkinWeightRawDataFromBuffer(const uint8* InSkinWeightData, uint32 InNumVertices)
{
	DataVertexBuffer.CopyDataFromBuffer(InSkinWeightData, InNumVertices);
}
