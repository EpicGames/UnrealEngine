// Copyright Epic Games, Inc. All Rights Reserved.

#include "Engine/SpecularProfile.h"
#include "Engine/Texture2D.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "Math/Float16.h"
#include "Rendering/BurleyNormalizedSSS.h"
#include "EngineModule.h"
#include "RenderTargetPool.h"
#include "PixelShaderUtils.h"
#include "RenderingThread.h"
#include "Rendering/Texture2DResource.h"
#include "RenderGraphResources.h"
#include "RenderGraphBuilder.h"
#include "SubstrateDefinitions.h"
#include "ShaderCompilerCore.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SpecularProfile)

DEFINE_LOG_CATEGORY_STATIC(LogSpecularProfile, Log, All);

///////////////////////////////////////////////////////////////////////////////////////////////////

static TAutoConsoleVariable<int32> CVarSpecularProfileResolution(
	TEXT("r.Substrate.SpecularProfile.Resolution"),
	64,
	TEXT("The resolution of the specular profile texture.\n"),
	ECVF_RenderThreadSafe
);

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
static TAutoConsoleVariable<int32> CVarSpecularProfileForceUpdate(
	TEXT("r.Substrate.SpecularProfile.ForceUpdate"),
	0,
	TEXT("0: Only update the specular profile as needed.\n")
	TEXT("1: Force to update the specular profile every frame for debugging.\n"),
	ECVF_Cheat | ECVF_RenderThreadSafe);
#endif

static bool ForceUpdateSpecularProfile()
{
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	return (CVarSpecularProfileForceUpdate.GetValueOnAnyThread() == 1);
#else
	return false;
#endif
}

///////////////////////////////////////////////////////////////////////////////////////////////////

static bool IsSpecularProfileSupport(EShaderPlatform InShaderPlatform)
{
	return IsFeatureLevelSupported(InShaderPlatform, ERHIFeatureLevel::ES3_1) && Substrate::IsSpecularProfileEnabled(InShaderPlatform);
}

class FSpecularProfileCopyCS : public FGlobalShader
{
public:
	DECLARE_GLOBAL_SHADER(FSpecularProfileCopyCS);
	SHADER_USE_PARAMETER_STRUCT(FSpecularProfileCopyCS, FGlobalShader);

	class FProcedural : SHADER_PERMUTATION_BOOL("PERMUTATION_PROCEDURAL");
	class FComponentCount : SHADER_PERMUTATION_SPARSE_INT("PERMUTATION_COMPONENT_COUNT", 3, 4);
	using FPermutationDomain = TShaderPermutationDomain<FProcedural, FComponentCount>;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_TEXTURE(Texture2D, SourceTexture)
		SHADER_PARAMETER_SAMPLER(SamplerState, SourceSampler)
		SHADER_PARAMETER(FIntPoint, SourceResolution)
		SHADER_PARAMETER(FIntPoint, TargetResolution)
		SHADER_PARAMETER(uint32, SourceMipCount)
		SHADER_PARAMETER(uint32, TargetIndex)
		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer, ViewColorBuffer)
		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer, LightColorBuffer)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D, TargetTexture)
	END_SHADER_PARAMETER_STRUCT()

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return IsSpecularProfileSupport(Parameters.Platform);
	}
	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("SHADER_COPY"), 1);
	}
};

IMPLEMENT_GLOBAL_SHADER(FSpecularProfileCopyCS, "/Engine/Private/SpecularProfile.usf", "MainCS", SF_Compute);

///////////////////////////////////////////////////////////////////////////////////////////////////

class FSpecularProfileConvolveCS : public FGlobalShader
{
public:
	DECLARE_GLOBAL_SHADER(FSpecularProfileConvolveCS);
	SHADER_USE_PARAMETER_STRUCT(FSpecularProfileConvolveCS, FGlobalShader);

	class FReadWriteOnArray : SHADER_PERMUTATION_BOOL("PERMUTATION_READ_WRITE_ON_ARRAY");
	class FComponentCount : SHADER_PERMUTATION_SPARSE_INT("PERMUTATION_COMPONENT_COUNT", 3, 4);
	using FPermutationDomain = TShaderPermutationDomain<FReadWriteOnArray, FComponentCount>;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER(uint32, SampleCount)
		SHADER_PARAMETER(FIntPoint, Resolution)
		SHADER_PARAMETER(uint32, ProfileCount)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2DArray, SourceTexture)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2DArray, TargetTexture)
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer, ProfileSignBuffer)
	END_SHADER_PARAMETER_STRUCT()

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		if (IsSpecularProfileSupport(Parameters.Platform))
		{
			const FPermutationDomain PermutationVector(Parameters.PermutationId);
			return !IsOpenGLPlatform(Parameters.Platform) || !PermutationVector.Get<FReadWriteOnArray>();
		}
		return false;
	}
	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("SHADER_CONVOLVE"), 1);
		OutEnvironment.CompilerFlags.Add(CFLAG_AllowTypedUAVLoads);		
	}
};

IMPLEMENT_GLOBAL_SHADER(FSpecularProfileConvolveCS, "/Engine/Private/SpecularProfile.usf", "MainCS", SF_Compute);

///////////////////////////////////////////////////////////////////////////////////////////////////
// FSpecularProfileTextureManager

// render thread
class FSpecularProfileTextureManager : public FRenderResource
{
public:
	// constructor
	FSpecularProfileTextureManager();

	// destructor
	~FSpecularProfileTextureManager();

	// convenience, can be optimized 
	// @param Profile must not be 0, game thread pointer, do not dereference, only for comparison
	int32 AddOrUpdateProfile(const USpecularProfile* InProfile, const FGuid& InGuid, const FSpecularProfileStruct InSettings, const FTextureReference* InTexture);

	// O(n) n is a small number
	// @param InProfile must not be 0, game thread pointer, do not dereference, only for comparison
	// @return AllocationId INDEX_NONE: no allocation, should be deallocated with DeallocateSpecularProfile()
	int32 AddProfile(const USpecularProfile* InProfile, const FGuid& InGuid, const FSpecularProfileStruct Settings, const FTextureReference* InTexture);

	// O(n) to find the element, n is the SSProfile count and usually quite small
	void RemoveProfile(const USpecularProfile* InProfile);

	// @param InProfile must not be 0, game thread pointer, do not dereference, only for comparison
	void UpdateProfile(const USpecularProfile* InProfile, const FSpecularProfileStruct InSettings, const FTextureReference* InTexture);

	// @param InProfile must not be 0, game thread pointer, do not dereference, only for comparison
	void UpdateProfile(int32 AllocationId, const FSpecularProfileStruct Settings, const FTextureReference* InTexture);

	// return the parameter name for a given profile
	FName GetParameterName(const USpecularProfile* InProfile) const;

	// Return the parameterization sign
	float GetParameterizationSign(int32 AllocationId) const;

	// @return can be nullptr if there is no SpecularProfile
	IPooledRenderTarget* GetAtlasTexture(FRDGBuilder& GraphBuilder, EShaderPlatform ShaderPlatform);
	IPooledRenderTarget* GetAtlasTexture();

	//~ Begin FRenderResource Interface.
	/**
		* Release textures when device is lost/destroyed.
		*/
	virtual void ReleaseRHI() override;

	// @param InProfile must not be 0, game thread pointer, do not dereference, only for comparison
	// @return INDEX_NONE if not found
	int32 FindAllocationId(const USpecularProfile* InProfile) const;

private:

	struct FProfileEntry
	{
		FSpecularProfileStruct Settings;
		const USpecularProfile* Profile = nullptr; // Game thread pointer! Do not dereference, only for comparison.
		const FTextureReference* Texture = nullptr;
		FIntPoint CachedResolution = FIntPoint::ZeroValue;
		FName ParameterName;
	};
	TArray<FProfileEntry> SpecularProfileEntries;
};

// Global resources - lives on the render thread
TGlobalResource<FSpecularProfileTextureManager> GSpecularProfileTextureManager;

// SpecularProfile atlas storing several texture profiles or 0 if there is no user
static TRefCountPtr<IPooledRenderTarget> GSpecularProfileTextureAtlas;

FSpecularProfileTextureManager::FSpecularProfileTextureManager()
{
	check(IsInGameThread());

	// add element 0, it is used as default profile
	FProfileEntry& Entry = SpecularProfileEntries.AddDefaulted_GetRef(); 
	Entry.Settings = FSpecularProfileStruct(); 
	Entry.Profile = nullptr; 
	Entry.Texture = nullptr; 
}

FSpecularProfileTextureManager::~FSpecularProfileTextureManager()
{
}

int32 FSpecularProfileTextureManager::AddOrUpdateProfile(const USpecularProfile* InProfile, const FGuid& Guid, const FSpecularProfileStruct InSettings, const FTextureReference* InTexture)
{
	check(InProfile);

	int32 AllocationId = FindAllocationId(InProfile); 
		
	if (AllocationId != INDEX_NONE)
	{
		UpdateProfile(AllocationId, InSettings, InTexture);
	}
	else
	{
		AllocationId = AddProfile(InProfile, Guid, InSettings, InTexture);
	}

	return AllocationId;
}

int32 FSpecularProfileTextureManager::AddProfile(const USpecularProfile* InProfile, const FGuid& Guid, const FSpecularProfileStruct InSettings, const FTextureReference* InTexture)
{
	check(InProfile);
	check(FindAllocationId(InProfile) == INDEX_NONE);

	int32 RetAllocationId = INDEX_NONE;
	{
		for (int32 i = 1; i < SpecularProfileEntries.Num(); ++i)
		{
			if (SpecularProfileEntries[i].Profile == nullptr)
			{
				RetAllocationId = i;

				FProfileEntry& Entry= SpecularProfileEntries[RetAllocationId];
				Entry.Profile 		= InProfile;
				Entry.ParameterName = FName(TEXT("__SpecularProfile") + Guid.ToString());
				break;
			}
		}

		if (RetAllocationId == INDEX_NONE)
		{
			RetAllocationId = SpecularProfileEntries.Num();

			FProfileEntry& Entry= SpecularProfileEntries.AddDefaulted_GetRef(); 			
			Entry.Profile 		= InProfile;
			Entry.ParameterName = FName(TEXT("__SpecularProfile") + Guid.ToString());
		}
	}

	UpdateProfile(RetAllocationId, InSettings, InTexture);

	return RetAllocationId;
}

void FSpecularProfileTextureManager::RemoveProfile(const USpecularProfile* InProfile)
{
	int32 AllocationId = FindAllocationId(InProfile);

	if (AllocationId != INDEX_NONE)
	{
		// >0 as 0 is used as default profile which should never be removed
		check(AllocationId > 0);
		check(SpecularProfileEntries[AllocationId].Profile == InProfile);

		// make it available for reuse
		SpecularProfileEntries[AllocationId].Profile = nullptr;
		SpecularProfileEntries[AllocationId].Settings.Invalidate();
		SpecularProfileEntries[AllocationId].Texture = nullptr;
		SpecularProfileEntries[AllocationId].ParameterName = FName();
	}
}

void FSpecularProfileTextureManager::UpdateProfile(const USpecularProfile* InProfile, const FSpecularProfileStruct InSettings, const FTextureReference* InTexture) 
{ 
	UpdateProfile(FindAllocationId(InProfile), InSettings, InTexture); 
}

void FSpecularProfileTextureManager::UpdateProfile(int32 AllocationId, const FSpecularProfileStruct Settings, const FTextureReference* InTexture)
{
	check(IsInRenderingThread());

	if (AllocationId != INDEX_NONE)
	{
		check(AllocationId < SpecularProfileEntries.Num());
		SpecularProfileEntries[AllocationId].Settings = Settings;
		SpecularProfileEntries[AllocationId].Texture = InTexture;
		GSpecularProfileTextureAtlas.SafeRelease();
	}
}

FName FSpecularProfileTextureManager::GetParameterName(const USpecularProfile* InProfile) const
{
	const int32 AllocationId = FindAllocationId(InProfile);
	if (AllocationId != INDEX_NONE)
	{
		// make it available for reuse
		return SpecularProfileEntries[AllocationId].ParameterName;
	}
	return FName();
}

float FSpecularProfileTextureManager::GetParameterizationSign(int32 AllocationId) const
{
	float OutSign = 1.f;
	if (AllocationId != INDEX_NONE)
	{
		OutSign = SpecularProfileEntries[AllocationId].Settings.Format == ESpecularProfileFormat::HalfVector ? -1.f : 1.f;
	}
	return OutSign;
}

IPooledRenderTarget* FSpecularProfileTextureManager::GetAtlasTexture()
{
	return GSpecularProfileTextureAtlas;
}

IPooledRenderTarget* FSpecularProfileTextureManager::GetAtlasTexture(FRDGBuilder& GraphBuilder, EShaderPlatform ShaderPlatform)
{
	if (!Substrate::IsSubstrateEnabled() || !IsSpecularProfileSupport(ShaderPlatform))
	{
		return nullptr;
	}

	const uint32 LayerCount = SpecularProfileEntries.Num();

	// Since reference texture can be streamed/loaded progressively, we track if textures have been updated to update the LUT
	bool bForceUpdate = ForceUpdateSpecularProfile();
	for (uint32 LayerIt = 0; LayerIt < LayerCount && !bForceUpdate; ++LayerIt)
	{
		if (SpecularProfileEntries[LayerIt].Texture)
		{
			const FTextureReferenceRHIRef& TextureRHI = SpecularProfileEntries[LayerIt].Texture->TextureReferenceRHI;
			if (TextureRHI->GetDesc().Extent != SpecularProfileEntries[LayerIt].CachedResolution)
			{
				bForceUpdate = true;
				break;
			}
		}
	}

	if (bForceUpdate)
	{
		GSpecularProfileTextureAtlas.SafeRelease();
		GSpecularProfileTextureAtlas = nullptr;
	}

	if (GSpecularProfileTextureAtlas == nullptr)
	{
		FRHICommandListImmediate& RHICmdList = GraphBuilder.RHICmdList;

		// Each row of the texture contains SSS parameters, followed by 3 precomputed kernels. Texture must be wide enough to fit all data.
		const uint32 Resolution = CVarSpecularProfileResolution.GetValueOnRenderThread();
		check(LayerCount);

		RDG_EVENT_SCOPE(GraphBuilder, "SpecularProfile");

		// 1. Create atlas texture
		// Check if we can do typed UAV loads on pixel formats with optional UAV load support
		const bool bR11G11B10Supported = GPixelFormats[PF_FloatR11G11B10].Supported && UE::PixelFormat::HasCapabilities(PF_FloatR11G11B10, EPixelFormatCapabilities::TextureFilterable | EPixelFormatCapabilities::UAV);
		const bool bR11G11B10TypedLoadSupported = bR11G11B10Supported && UE::PixelFormat::HasCapabilities(PF_FloatR11G11B10, EPixelFormatCapabilities::TypedUAVLoad);
		const bool bRGBA16TypedLoadSupported = UE::PixelFormat::HasCapabilities(PF_FloatRGBA, EPixelFormatCapabilities::TypedUAVLoad);

		// Certain platforms don't support read/write on the same texture array, even if this done safely on different slices.
		// We fall back to an SRV load during convolution if we can't support this, or if neither pixel format supports typed UAV loads.
		const bool bConvolveSupportsTypedUAVLoad = !IsOpenGLPlatform(ShaderPlatform) && (bR11G11B10TypedLoadSupported || bRGBA16TypedLoadSupported);

		const EPixelFormat Format = bConvolveSupportsTypedUAVLoad ?
			(bR11G11B10TypedLoadSupported ? PF_FloatR11G11B10 : PF_FloatRGBA) :
			(bR11G11B10Supported ? PF_FloatR11G11B10 : PF_FloatRGBA);

		const int32 ComponentCount = GPixelFormats[Format].NumComponents;

		FPooledRenderTargetDesc Desc(FPooledRenderTargetDesc::Create2DArrayDesc(FIntPoint(Resolution, Resolution), Format, FClearValueBinding::None, TexCreate_None, TexCreate_UAV | TexCreate_ShaderResource, false, LayerCount * SUBSTRATE_SPECULAR_PROFILE_ENTRY_COUNT));
		GRenderTargetPool.FindFreeElement(RHICmdList, Desc, GSpecularProfileTextureAtlas, TEXT("SpecularProfileTexture"));
		const auto GlobalShaderMap = GetGlobalShaderMap(ShaderPlatform);
		FRDGTextureRef SpecularProfileTexture = GraphBuilder.RegisterExternalTexture(GSpecularProfileTextureAtlas, TEXT("SpecularProfileTexture"));
		
		// 2. Optionally clear profiles texture
		if (bForceUpdate)
		{
			AddClearUAVPass(GraphBuilder, GraphBuilder.CreateUAV(SpecularProfileTexture), 0.f);
		}

		// 3. Fill in profiles
		FRDGTextureUAVRef SpecularProfileUAVSkipBarrier = GraphBuilder.CreateUAV(SpecularProfileTexture, ERDGUnorderedAccessViewFlags::SkipBarrier);
		for (uint32 LayerIt = 0; LayerIt < LayerCount; ++LayerIt)
		{
			const FSpecularProfileStruct Data = SpecularProfileEntries[LayerIt].Settings;
			const FTextureReference* Texture = SpecularProfileEntries[LayerIt].Texture;

			FSpecularProfileCopyCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FSpecularProfileCopyCS::FParameters>();
			PassParameters->SourceMipCount = Texture ? FMath::Max(1u, uint32(Texture->TextureReferenceRHI->GetDesc().NumMips)) : 0;
			PassParameters->SourceTexture = Texture ? Texture->TextureReferenceRHI->GetReferencedTexture() : nullptr;
			PassParameters->SourceSampler = TStaticSamplerState<SF_Point, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
			PassParameters->SourceResolution = SpecularProfileTexture->Desc.Extent;
			PassParameters->TargetResolution = SpecularProfileTexture->Desc.Extent;
			PassParameters->TargetTexture = SpecularProfileUAVSkipBarrier;
			PassParameters->TargetIndex = LayerIt;

			struct FLinearColor16 
			{
				FLinearColor16() {}
				FLinearColor16(const FLinearColor& In) { R = In.R; G = In.G; B = In.B; A = In.A; }
				FFloat16 R;
				FFloat16 G;
				FFloat16 B;
				FFloat16 A;
			};

			TArray<FLinearColor16> ViewColorData;
			TArray<FLinearColor16> LightColorData;
			ViewColorData.SetNum(PassParameters->TargetResolution.X);
			LightColorData.SetNum(PassParameters->TargetResolution.Y);
			for (uint32 It=0; It < Resolution; ++It)
			{
				ViewColorData[It]  = FLinearColor16(Data.ViewColor.GetLinearColorValue(float(It + 0.5f) / Resolution));
				LightColorData[It] = FLinearColor16(Data.LightColor.GetLinearColorValue(float(It + 0.5f) / Resolution));
			}

			FRDGBufferRef ViewColorBuffer = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateBufferDesc(8, Resolution), TEXT("ViewColor"), ERDGBufferFlags::MultiFrame);
			GraphBuilder.QueueBufferUpload(ViewColorBuffer, ViewColorData.GetData(), 8*ViewColorData.Num(), ERDGInitialDataFlags::None);
			PassParameters->ViewColorBuffer = GraphBuilder.CreateSRV(ViewColorBuffer, PF_FloatRGBA);

			FRDGBufferRef LightColorBuffer = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateBufferDesc(8, Resolution), TEXT("LightColor"), ERDGBufferFlags::MultiFrame);
			GraphBuilder.QueueBufferUpload(LightColorBuffer, LightColorData.GetData(), 8*LightColorData.Num(), ERDGInitialDataFlags::None);
			PassParameters->LightColorBuffer = GraphBuilder.CreateSRV(LightColorBuffer, PF_FloatRGBA);

			FSpecularProfileCopyCS::FPermutationDomain PermutationVector;
			PermutationVector.Set<FSpecularProfileCopyCS::FProcedural>(Data.IsProcedural());
			PermutationVector.Set<FSpecularProfileCopyCS::FComponentCount>(ComponentCount);
			TShaderMapRef<FSpecularProfileCopyCS> Shader(GlobalShaderMap, PermutationVector);
			FComputeShaderUtils::AddPass(GraphBuilder, 
				RDG_EVENT_NAME("SpecularProfile::CopyTexture"), 
				Shader, 
				PassParameters, 
				FIntVector(FMath::DivideAndRoundUp(PassParameters->TargetResolution.X, 8), FMath::DivideAndRoundUp(PassParameters->TargetResolution.Y, 8), 1));

			SpecularProfileEntries[LayerIt].CachedResolution = Texture ? Texture->TextureReferenceRHI->GetDesc().Extent : FIntPoint(0,0);
		}

		// Convolve all the profiles
		if (LayerCount > 0)
		{
			FRDGTextureRef SourceSpecularProfileTexture = nullptr;
			if (!bConvolveSupportsTypedUAVLoad)
			{
				const FIntPoint Extent = SpecularProfileTexture->Desc.Extent;
				SourceSpecularProfileTexture = GraphBuilder.CreateTexture(SpecularProfileTexture->Desc, TEXT("ConvolvedSpecularProfileTexture"));
				AddCopyTexturePass(GraphBuilder, SpecularProfileTexture, SourceSpecularProfileTexture);
			}

			TArray<float> ProfileSign;
			ProfileSign.SetNum(LayerCount);
			for (uint32 LayerIt=0; LayerIt < LayerCount; ++LayerIt)
			{
				ProfileSign[LayerIt] = GetParameterizationSign(LayerIt);
			}
			FRDGBufferRef ProfileSignBuffer = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateStructuredDesc(sizeof(float), LayerCount), TEXT("SpecularProfileData"), ERDGBufferFlags::MultiFrame);
			GraphBuilder.QueueBufferUpload(ProfileSignBuffer, ProfileSign.GetData(), sizeof(float)*ProfileSign.Num(), ERDGInitialDataFlags::None);

			FSpecularProfileConvolveCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FSpecularProfileConvolveCS::FParameters>();
			PassParameters->SampleCount = 64u;
			PassParameters->ProfileCount = LayerCount;
			PassParameters->Resolution = SpecularProfileTexture->Desc.Extent;
			PassParameters->ProfileSignBuffer = GraphBuilder.CreateSRV(ProfileSignBuffer);
			PassParameters->SourceTexture = SourceSpecularProfileTexture;
			PassParameters->TargetTexture = GraphBuilder.CreateUAV(SpecularProfileTexture);

			FSpecularProfileConvolveCS::FPermutationDomain PermutationVector;
			PermutationVector.Set<FSpecularProfileConvolveCS::FReadWriteOnArray>(PassParameters->SourceTexture == nullptr);
			PermutationVector.Set<FSpecularProfileConvolveCS::FComponentCount>(ComponentCount);
			TShaderMapRef<FSpecularProfileConvolveCS> Shader(GlobalShaderMap, PermutationVector);
			FComputeShaderUtils::AddPass(GraphBuilder, 
				RDG_EVENT_NAME("SpecularProfile::Convolve"), 
				Shader, 
				PassParameters, 
				FIntVector(FMath::DivideAndRoundUp(PassParameters->Resolution.X, 8), FMath::DivideAndRoundUp(PassParameters->Resolution.Y, 8), LayerCount));
		}

		// Transit texture to SRV for letting the non-RDG resource to be bound later by the various passes
		FRDGExternalAccessQueue ExternalAccessQueue;
		ExternalAccessQueue.Add(SpecularProfileTexture);
		ExternalAccessQueue.Submit(GraphBuilder);
	}
	return GSpecularProfileTextureAtlas;
}

void FSpecularProfileTextureManager::ReleaseRHI()
{
	GSpecularProfileTextureAtlas.SafeRelease();
}

int32 FSpecularProfileTextureManager::FindAllocationId(const USpecularProfile* InProfile) const
{
	// we start at 1 because [0] is the default profile and always [0].Profile = 0 so we don't need to iterate that one
	for (int32 i = 1; i < SpecularProfileEntries.Num(); ++i)
	{
		if (SpecularProfileEntries[i].Profile == InProfile)
		{
			return i;
		}
	}
	return INDEX_NONE;
}

///////////////////////////////////////////////////////////////////////////////////////////////////
// FSpecularProfileStruct

FSpecularProfileStruct::FSpecularProfileStruct()
{
	ViewColor.ColorCurves[0].AddKey(0, 1.f);
	ViewColor.ColorCurves[1].AddKey(0, 1.f);
	ViewColor.ColorCurves[2].AddKey(0, 1.f);
	ViewColor.ColorCurves[0].AddKey(1, 1.f);
	ViewColor.ColorCurves[1].AddKey(1, 1.f);
	ViewColor.ColorCurves[2].AddKey(1, 1.f);

	LightColor.ColorCurves[0].AddKey(0, 1.f);
	LightColor.ColorCurves[1].AddKey(0, 1.f);
	LightColor.ColorCurves[2].AddKey(0, 1.f);
	LightColor.ColorCurves[0].AddKey(1, 1.f);
	LightColor.ColorCurves[1].AddKey(1, 1.f);
	LightColor.ColorCurves[2].AddKey(1, 1.f);

	Format = ESpecularProfileFormat::ViewLightVector;
	Texture = nullptr;
}

///////////////////////////////////////////////////////////////////////////////////////////////////
// USpecularProfile

USpecularProfile::USpecularProfile(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

void USpecularProfile::BeginDestroy()
{
	USpecularProfile* Ref = this;
	ENQUEUE_RENDER_COMMAND(RemoveSpecularProfile)(
	[Ref](FRHICommandList& RHICmdList)
	{
		GSpecularProfileTextureManager.RemoveProfile(Ref);
	});

	Super::BeginDestroy();
}

#if WITH_EDITOR
void USpecularProfile::PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	const FSpecularProfileStruct LocalSettings = this->Settings;
	USpecularProfile* LocalProfile = this;
	GetRendererModule().InvalidatePathTracedOutput();

	const FTextureReference* LocalTextureResource = nullptr;
	if (!LocalSettings.IsProcedural())
	{
		LocalTextureResource = &LocalSettings.Texture->TextureReference;
	}

	ENQUEUE_RENDER_COMMAND(UpdateSpecularProfile)(
	[LocalSettings, LocalProfile, LocalTextureResource](FRHICommandListImmediate& RHICmdList)
	{
		// any changes to the setting require an update of the texture
		GSpecularProfileTextureManager.UpdateProfile(LocalProfile, LocalSettings, LocalTextureResource);
	});
}
#endif // WITH_EDITOR

void USpecularProfile::PostDuplicate(EDuplicateMode::Type DuplicateMode)
{
	Super::PostDuplicate(DuplicateMode);

	// When a Specular Profile asset is duplicated/copied pasted (e.g. from the asset browser), we want the guid to be regenerated.
	Guid = FGuid::NewGuid();
}

///////////////////////////////////////////////////////////////////////////////////////////////////
// Public API

namespace SpecularProfile
{
	FName GetSpecularProfileParameterName(const USpecularProfile* In)
	{
		return GSpecularProfileTextureManager.GetParameterName(In);
	}

	float GetSpecularProfileId(const USpecularProfile* In)
	{
		// No profile specified means we use the default one (constant one)
		int32 AllocationId = 0;
		if (In)
		{
			// can be optimized (cached)
			AllocationId = GSpecularProfileTextureManager.FindAllocationId(In);
		}
		return GSpecularProfileTextureManager.GetParameterizationSign(AllocationId) * AllocationId / 255.0f;
	}
	
	int32 AddOrUpdateProfile(const USpecularProfile* InProfile, const FGuid& InGuid, const FSpecularProfileStruct InSettings, const FTextureReference* InTexture)
	{
		return GSpecularProfileTextureManager.AddOrUpdateProfile(InProfile, InGuid, InSettings, InTexture);
	}

	FRHITexture* GetSpecularProfileTextureAtlas()
	{
		return GSpecularProfileTextureAtlas ? GSpecularProfileTextureAtlas->GetRHI() : nullptr;
	}

	FRHITexture* GetSpecularProfileTextureAtlasWithFallback()
	{
		return GSpecularProfileTextureAtlas ? GSpecularProfileTextureAtlas->GetRHI() : static_cast<FRHITexture*>(GBlackTexture->TextureRHI);
	}

	void UpdateSpecularProfileTextureAtlas(FRDGBuilder& GraphBuilder, EShaderPlatform ShaderPlatform)
	{
		GSpecularProfileTextureManager.GetAtlasTexture(GraphBuilder, ShaderPlatform);
	}

	
	static FName CreateSpecularProfileParameterName(const FGuid& InGuid)
	{
		return FName(TEXT("__SpecularProfile") + InGuid.ToString());
	}

	FName CreateSpecularProfileParameterName(USpecularProfile* InProfile)
	{
		return InProfile ? CreateSpecularProfileParameterName(InProfile->Guid) : FName();
	}
} // namespace SpecularProfile
