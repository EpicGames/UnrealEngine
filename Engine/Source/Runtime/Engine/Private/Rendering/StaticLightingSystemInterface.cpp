// Copyright Epic Games, Inc. All Rights Reserved.

#include "Rendering/StaticLightingSystemInterface.h"
#include "Components/LightComponent.h"
#include "Components/PrimitiveComponent.h"

#define LOCTEXT_NAMESPACE "StaticLightingSystem"

FStaticLightingSystemInterface* FStaticLightingSystemInterface::Interface = nullptr;

FStaticLightingSystemInterface::FPrimitiveComponentBasedSignature FStaticLightingSystemInterface::OnPrimitiveComponentRegistered;
FStaticLightingSystemInterface::FPrimitiveComponentBasedSignature FStaticLightingSystemInterface::OnPrimitiveComponentUnregistered;
FStaticLightingSystemInterface::FLightComponentBasedSignature FStaticLightingSystemInterface::OnLightComponentRegistered;
FStaticLightingSystemInterface::FLightComponentBasedSignature FStaticLightingSystemInterface::OnLightComponentUnregistered;
FStaticLightingSystemInterface::FStationaryLightChannelReassignmentSignature FStaticLightingSystemInterface::OnStationaryLightChannelReassigned;
FStaticLightingSystemInterface::FLightmassImportanceVolumeModifiedSignature FStaticLightingSystemInterface::OnLightmassImportanceVolumeModified;
FStaticLightingSystemInterface::FMaterialInvalidationSignature FStaticLightingSystemInterface::OnMaterialInvalidated;
FSimpleMulticastDelegate FStaticLightingSystemInterface::OnSkyAtmosphereModified;

FStaticLightingSystemInterface* FStaticLightingSystemInterface::Get()
{
	if (!Interface)
	{
		Interface = new FStaticLightingSystemInterface();
	}

	return Interface;
}

const FMeshMapBuildData* FStaticLightingSystemInterface::GetPrimitiveMeshMapBuildData(const UPrimitiveComponent* Component, int32 LODIndex /* = 0 */)
{
	if (Get()->GetPreferredImplementation())
	{
		if (Component->GetWorld())
		{
			if (Get()->GetPreferredImplementation()->GetStaticLightingSystemForWorld(Component->GetWorld()))
			{
				return Get()->GetPreferredImplementation()->GetStaticLightingSystemForWorld(Component->GetWorld())->GetPrimitiveMeshMapBuildData(Component, LODIndex);
			}
		}
	}

	return nullptr;
}

const FLightComponentMapBuildData* FStaticLightingSystemInterface::GetLightComponentMapBuildData(const ULightComponent* Component)
{
	if (Get()->GetPreferredImplementation())
	{
		if (Component->GetWorld())
		{
			if (Get()->GetPreferredImplementation()->GetStaticLightingSystemForWorld(Component->GetWorld()))
			{
				return Get()->GetPreferredImplementation()->GetStaticLightingSystemForWorld(Component->GetWorld())->GetLightComponentMapBuildData(Component);
			}
		}
	}

	return nullptr;
}

const FPrecomputedVolumetricLightmap* FStaticLightingSystemInterface::GetPrecomputedVolumetricLightmap(UWorld* World)
{
	if (Get()->GetPreferredImplementation())
	{
		if (Get()->GetPreferredImplementation()->GetStaticLightingSystemForWorld(World))
		{
			return Get()->GetPreferredImplementation()->GetStaticLightingSystemForWorld(World)->GetPrecomputedVolumetricLightmap();
		}
	}

	return nullptr;
}

void FStaticLightingSystemInterface::EditorTick()
{
	if (Get()->GetPreferredImplementation())
	{
		Get()->GetPreferredImplementation()->EditorTick();
	}
}

// For editor -game
void FStaticLightingSystemInterface::GameTick(float DeltaSeconds)
{
	EditorTick();
}

bool FStaticLightingSystemInterface::IsStaticLightingSystemRunning()
{
	if (Get()->GetPreferredImplementation())
	{
		return Get()->GetPreferredImplementation()->IsStaticLightingSystemRunning();
	}

	return false;
}

void FStaticLightingSystemInterface::RegisterImplementation(FName Name, IStaticLightingSystemImpl* StaticLightingSystemImpl)
{
	check(IsInGameThread());
	check(Implementations.Find(Name) == nullptr);
	Implementations.Add(Name, StaticLightingSystemImpl);
}

void FStaticLightingSystemInterface::UnregisterImplementation(FName Name)
{
	check(IsInGameThread());
	check(Implementations.Find(Name) != nullptr);
	Implementations.Remove(Name);
}

IStaticLightingSystemImpl* FStaticLightingSystemInterface::GetPreferredImplementation()
{
	static const FName NameGPULightmass("GPULightmass");
	if (IStaticLightingSystemImpl** Found = Implementations.Find(NameGPULightmass))
	{
		return *Found;
	}
	
	if (Implementations.Num() > 0)
	{
		return Implementations.CreateIterator().Value();
	}

	return nullptr;
}

#undef LOCTEXT_NAMESPACE
