// Copyright Epic Games, Inc. All Rights Reserved.

#include "Rendering/StaticMeshVertexBuffer.h"

#include "Components.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "EngineUtils.h"
#include "LocalVertexFactory.h"
#include "MeshUVChannelInfo.h"
#include "ProfilingDebugging/LoadTimeTracker.h"
#include "RHIResourceReplace.h"
#include "StaticMeshVertexData.h"

	
static uint32 GetNumTexCoordInVertexBuffer(uint32 NumTexCoords, uint32 NumVertices)
{
	// Increase size with single vertex in case we have odd number of texcoords because the vertex declaration
	// will declare the stream as half4/float4 while it only contains a half2/float2 and the last element can in theory
	// be out of bounds then
	// Vertex streams are all created with half4/float4 to reduce the number of PSO permutations coming from different texcoords in the vertex declaration
	uint32 Padding = NumVertices > 0 ? NumTexCoords % 2 : 0;
	return (NumVertices + Padding) * NumTexCoords;
}

/** The implementation of the static mesh texture coordinate vertex data storage type which takes extra padding into account. */
template<typename VertexDataType>
class TStaticTexCoordVertexData :
	public TStaticMeshVertexData<VertexDataType>
{
private:

	uint32 NumTexCoords;

public:

	TStaticTexCoordVertexData(uint32 InNumTexCoords, bool InNeedsCPUAccess=false)
		: TStaticMeshVertexData<VertexDataType>(InNeedsCPUAccess), NumTexCoords(InNumTexCoords)
	{
	}

	void ResizeBuffer(uint32 NumVertices, EResizeBufferFlags BufferFlags = EResizeBufferFlags::None) override
	{
		uint32 NumTexCoordData = GetNumTexCoordInVertexBuffer(NumTexCoords, NumVertices);
		TStaticMeshVertexData<VertexDataType>::ResizeBuffer(NumTexCoordData);

		// When NumTexCoords is odd, GetNumTexCoordInVertexBuffer adds a padding vertex
		// for GPU half4/float4 alignment. Zero it so DDC serialization is deterministic.
		VertexDataType* Data = reinterpret_cast<VertexDataType*>(this->GetDataPointer());
		for (uint32 Index = NumVertices * NumTexCoords; Index < NumTexCoordData; Index++)
		{
			Data[Index] = VertexDataType();
		}
	}

	void Empty(uint32 NumVertices) override
	{
		TStaticMeshVertexData<VertexDataType>::Empty(GetNumTexCoordInVertexBuffer(NumTexCoords, NumVertices));
	}
};

FStaticMeshVertexBuffer::FStaticMeshVertexBuffer() :
	TangentsData(nullptr),
	TexcoordData(nullptr),
	TangentsDataPtr(nullptr),
	TexcoordDataPtr(nullptr),
	NumTexCoords(0),
	NumVertices(0),
	bUseFullPrecisionUVs(false),
	bUseHighPrecisionTangentBasis(false)
{}

FStaticMeshVertexBuffer::~FStaticMeshVertexBuffer()
{
	CleanUp();
}

/** Delete existing resources */
void FStaticMeshVertexBuffer::CleanUp()
{
	if (TangentsData)
	{
		delete TangentsData;
		TangentsData = nullptr;
	}
	if (TexcoordData)
	{
		delete TexcoordData;
		TexcoordData = nullptr;
	}
}

void FStaticMeshVertexBuffer::Init(uint32 InNumVertices, uint32 InNumTexCoords, bool bNeedsCPUAccess)
{
	NumTexCoords = InNumTexCoords;
	NumVertices = InNumVertices;
	NeedsCPUAccess = bNeedsCPUAccess;

	// Allocate the vertex data storage type.
	AllocateData(bNeedsCPUAccess);

	// Allocate the vertex data buffer.
	TangentsData->ResizeBuffer(NumVertices);
	TangentsDataPtr = NumVertices ? TangentsData->GetDataPointer() : nullptr;
	TexcoordData->ResizeBuffer(NumVertices * GetNumTexCoords());
	TexcoordDataPtr = NumVertices ? TexcoordData->GetDataPointer() : nullptr;
}

/**
* Initializes the buffer with the given vertices.
* @param InVertices - The vertices to initialize the buffer with.
* @param InNumTexCoords - The number of texture coordinate to store in the buffer.
*/
void FStaticMeshVertexBuffer::Init(const TArray<FStaticMeshBuildVertex>& InVertices, uint32 InNumTexCoords, const FStaticMeshVertexBufferFlags & InInitFlags)
{
	FConstMeshBuildVertexView VertexView = MakeConstMeshBuildVertexView(InVertices);
	Init(VertexView, InInitFlags);
}

/**
* Initializes the buffer with the given vertex view.
* @param InVertices - The vertices to initialize the buffer with.
* @param InNumTexCoords - The number of texture coordinate to store in the buffer.
*/
void FStaticMeshVertexBuffer::Init(const FConstMeshBuildVertexView& InVertices, const FStaticMeshVertexBufferFlags& InInitFlags)
{
	Init(InVertices.Position.Num(), InVertices.UVs.Num(), InInitFlags.bNeedsCPUAccess);

	// Copy the vertices into the buffer.
	for (int32 VertexIndex = 0; VertexIndex < InVertices.Position.Num(); VertexIndex++)
	{
		const uint32 DestVertexIndex = VertexIndex;
		SetVertexTangents(DestVertexIndex, InVertices.TangentX[VertexIndex], InVertices.TangentY[VertexIndex], InVertices.TangentZ[VertexIndex]);

		for (int32 UVIndex = 0; UVIndex < InVertices.UVs.Num(); UVIndex++)
		{
			SetVertexUV(DestVertexIndex, UVIndex, InVertices.UVs[UVIndex][VertexIndex], InInitFlags.bUseBackwardsCompatibleF16TruncUVs);
		}
	}
}

/**
* Initializes this vertex buffer with the contents of the given vertex buffer.
* @param InVertexBuffer - The vertex buffer to initialize from.
*/
void FStaticMeshVertexBuffer::Init(const FStaticMeshVertexBuffer& InVertexBuffer, bool bNeedsCPUAccess)
{
	NeedsCPUAccess = bNeedsCPUAccess;
	NumTexCoords = InVertexBuffer.GetNumTexCoords();
	NumVertices = InVertexBuffer.GetNumVertices();
	bUseFullPrecisionUVs = InVertexBuffer.GetUseFullPrecisionUVs();
	bUseHighPrecisionTangentBasis = InVertexBuffer.GetUseHighPrecisionTangentBasis();

	if (NumVertices)
	{
		AllocateData(bNeedsCPUAccess);
		{
			check(TangentsData->GetStride() == InVertexBuffer.TangentsData->GetStride());
			TangentsData->ResizeBuffer(NumVertices);
			TangentsDataPtr = TangentsData->GetDataPointer();
			const uint8* InData = InVertexBuffer.TangentsDataPtr;
			FMemory::Memcpy(TangentsDataPtr, InData, TangentsData->GetStride() * NumVertices);
		}
		{
			check(TexcoordData->GetStride() == InVertexBuffer.TexcoordData->GetStride());
			check(GetNumTexCoords() == InVertexBuffer.GetNumTexCoords());
			const uint8* InData = InVertexBuffer.TexcoordDataPtr;

			TexcoordData->ResizeBuffer(NumVertices * GetNumTexCoords());
			TexcoordDataPtr = TexcoordData->GetDataPointer();
			FMemory::Memcpy(TexcoordDataPtr, InData, TexcoordData->GetStride() * NumVertices * GetNumTexCoords());
		}
	}
}

void FStaticMeshVertexBuffer::AppendVertices( const FStaticMeshBuildVertex* Vertices, const uint32 NumVerticesToAppend, bool bUseBackwardsCompatibleF16TruncUVs)
{
	if ((TangentsData == nullptr || TexcoordData == nullptr) && NumVerticesToAppend > 0)
	{
		check(NumVertices == 0);
		NumTexCoords = 1;

		// Allocate the vertex data storage type if it has never been allocated before
		AllocateData(NeedsCPUAccess);
	}

	if( NumVerticesToAppend > 0 )
	{
		check( Vertices != nullptr );

		const uint32 FirstDestVertexIndex = NumVertices;
		NumVertices += NumVerticesToAppend;

		TangentsData->ResizeBuffer(NumVertices);
		TexcoordData->ResizeBuffer(NumVertices * GetNumTexCoords());

		if( NumVertices > 0 )
		{
			TangentsDataPtr = TangentsData->GetDataPointer();
			TexcoordDataPtr = TexcoordData->GetDataPointer();

			// Copy the vertices into the buffer.
			for( uint32 VertexIter = 0; VertexIter < NumVerticesToAppend; ++VertexIter )
			{
				const FStaticMeshBuildVertex& SourceVertex = Vertices[ VertexIter ];

				const uint32 DestVertexIndex = FirstDestVertexIndex + VertexIter;

				SetVertexTangents( DestVertexIndex, SourceVertex.TangentX, SourceVertex.TangentY, SourceVertex.TangentZ );
				for( uint32 UVIndex = 0; UVIndex < NumTexCoords; UVIndex++ )
				{
					SetVertexUV( DestVertexIndex, UVIndex, SourceVertex.UVs[ UVIndex ], bUseBackwardsCompatibleF16TruncUVs );
				}
			}
		}
	}
}


/**
* Serializer
*
* @param	Ar				Archive to serialize with
* @param	bNeedsCPUAccess	Whether the elements need to be accessed by the CPU
*/
void FStaticMeshVertexBuffer::Serialize(FArchive& Ar, bool bNeedsCPUAccess)
{
	NeedsCPUAccess = bNeedsCPUAccess;

	DECLARE_SCOPE_CYCLE_COUNTER(TEXT("FStaticMeshVertexBuffer::Serialize"), STAT_StaticMeshVertexBuffer_Serialize, STATGROUP_LoadTime);

	FStripDataFlags StripFlags(Ar, 0, FPackageFileVersion::CreateUE4Version(VER_UE4_STATIC_SKELETAL_MESH_SERIALIZATION_FIX));

	SerializeMetaData(Ar);

	if (Ar.IsLoading())
	{
		// Allocate the vertex data storage type.
		AllocateData(bNeedsCPUAccess);
	}

	if (!StripFlags.IsAudioVisualDataStripped() || Ar.IsCountingMemory())
	{
		if (TangentsData != nullptr)
		{
			// Serialize the vertex data.
			TangentsData->Serialize(Ar);

			// Make a copy of the vertex data pointer.
			TangentsDataPtr = NumVertices ? TangentsData->GetDataPointer() : nullptr;
		}

		if (TexcoordData != nullptr)
		{
			// Serialize the vertex data.
			TexcoordData->Serialize(Ar);

			// Make a copy of the vertex data pointer.
			TexcoordDataPtr = NumVertices ? TexcoordData->GetDataPointer() : nullptr;
		}
	}
}

void FStaticMeshVertexBuffer::SerializeMetaData(FArchive& Ar)
{
	Ar << NumTexCoords << NumVertices;
	Ar << bUseFullPrecisionUVs;
	Ar << bUseHighPrecisionTangentBasis;

	InitTangentAndTexCoordStrides();
}

void FStaticMeshVertexBuffer::SetMetaData(uint32 InNumTexCoords, uint32 InNumVertices, bool bInUseFullPrecisionUVs, bool bInUseHighPrecisionTangentBasis)
{
	NumTexCoords = InNumTexCoords;
	NumVertices = InNumVertices;
	bUseFullPrecisionUVs = bInUseFullPrecisionUVs;
	bUseHighPrecisionTangentBasis = bInUseHighPrecisionTangentBasis;

	InitTangentAndTexCoordStrides();
}

void FStaticMeshVertexBuffer::ClearMetaData()
{
	NumTexCoords = NumVertices = 0;
	bUseFullPrecisionUVs = false;
	bUseHighPrecisionTangentBasis = false;
	TangentsStride = TexcoordStride = 0;
}


/**
* Specialized assignment operator, only used when importing LOD's.
*/
void FStaticMeshVertexBuffer::operator=(const FStaticMeshVertexBuffer &Other)
{
	//VertexData doesn't need to be allocated here because Build will be called next,
	CleanUp();
	bUseFullPrecisionUVs = Other.bUseFullPrecisionUVs;
	bUseHighPrecisionTangentBasis = Other.bUseHighPrecisionTangentBasis;
}

FBufferRHIRef FStaticMeshVertexBuffer::CreateTangentsRHIBuffer(FRHICommandListBase& RHICmdList)
{
	return CreateRHIBuffer(RHICmdList, TangentsData, GetNumVertices(), BUF_Static | BUF_ShaderResource, TEXT("TangentsRHIBuffer"));
}

FBufferRHIRef FStaticMeshVertexBuffer::CreateTexCoordRHIBuffer(FRHICommandListBase& RHICmdList)
{
	uint32 ResourceCount = GetNumVertices() * GetNumTexCoords();
	return CreateRHIBuffer(RHICmdList, TexcoordData, ResourceCount, BUF_Static | BUF_ShaderResource, TEXT("TexCoordRHIBuffer"));
}

void FStaticMeshVertexBuffer::InitRHIForStreaming(
	FRHIBuffer* IntermediateTangentsBuffer,
	FRHIBuffer* IntermediateTexCoordBuffer,
	FRHIResourceReplaceBatcher& Batcher)
{
	check(TangentsVertexBuffer.VertexBufferRHI && TexCoordVertexBuffer.VertexBufferRHI);
	if (IntermediateTangentsBuffer)
	{
		Batcher.EnqueueReplace(TangentsVertexBuffer.VertexBufferRHI, IntermediateTangentsBuffer);
	}
	if (IntermediateTexCoordBuffer)
	{
		Batcher.EnqueueReplace(TexCoordVertexBuffer.VertexBufferRHI, IntermediateTexCoordBuffer);
	}
}

void FStaticMeshVertexBuffer::ReleaseRHIForStreaming(FRHIResourceReplaceBatcher& Batcher)
{
	check(TangentsVertexBuffer.VertexBufferRHI && TexCoordVertexBuffer.VertexBufferRHI);
	Batcher.EnqueueReplace(TangentsVertexBuffer.VertexBufferRHI, nullptr);
	Batcher.EnqueueReplace(TexCoordVertexBuffer.VertexBufferRHI, nullptr);
}

void FStaticMeshVertexBuffer::InitRHI(FRHICommandListBase& RHICmdList)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FStaticMeshVertexBuffer::InitRHI);
	SCOPED_LOADTIMER(FStaticMeshVertexBuffer_InitRHI);

	// When bAllowCPUAccess is true the meshes is likely going to be used for Niagara to spawn particles on mesh surface.
	// And it can be the case for CPU *and* GPU access: no differenciation today. That is why we create a SRV in this case.
	// This also avoid setting lots of states on all the members of all the different buffers used by meshes. Follow up: https://jira.it.epicgames.net/browse/UE-69376.

	const bool bHadTangentsData = TangentsData != nullptr;
	const bool bCreateTangentsSRV = bHadTangentsData && TangentsData->GetAllowCPUAccess();
	TangentsVertexBuffer.VertexBufferRHI = CreateTangentsRHIBuffer(RHICmdList);

	if (bCreateTangentsSRV || AreBufferSRVsAlwaysCreatedByDefault(GMaxRHIShaderPlatform))
	{
		TangentsSRV = GetOrCreateTangentsSRV(RHICmdList);
	}

	const bool bHadTexCoordData = TexcoordData != nullptr;
	const bool bCreateTexCoordSRV = bHadTexCoordData && TexcoordData->GetAllowCPUAccess();
	TexCoordVertexBuffer.VertexBufferRHI = CreateTexCoordRHIBuffer(RHICmdList);
	
	if (bCreateTexCoordSRV || AreBufferSRVsAlwaysCreatedByDefault(GMaxRHIShaderPlatform))
	{
		TextureCoordinatesSRV = GetOrCreateTexCoordsSRV(RHICmdList);	
	}
}

void FStaticMeshVertexBuffer::ReleaseRHI()
{
	TangentsSRV.SafeRelease();
	TextureCoordinatesSRV.SafeRelease();
	TangentsVertexBuffer.ReleaseRHI();
	TexCoordVertexBuffer.ReleaseRHI();
}

void FStaticMeshVertexBuffer::InitResource(FRHICommandListBase& RHICmdList)
{
	FRenderResource::InitResource(RHICmdList);
	TangentsVertexBuffer.InitResource(RHICmdList);
	TexCoordVertexBuffer.InitResource(RHICmdList);
}

void FStaticMeshVertexBuffer::ReleaseResource()
{
	FRenderResource::ReleaseResource();
	TangentsVertexBuffer.ReleaseResource();
	TexCoordVertexBuffer.ReleaseResource();
}

void FStaticMeshVertexBuffer::AllocateData(bool bNeedsCPUAccess /*= true*/)
{
	// Clear any old VertexData before allocating.
	CleanUp();

	uint32 VertexStride = 0;
	if (GetUseHighPrecisionTangentBasis())
	{
		typedef TStaticMeshVertexTangentDatum<typename TStaticMeshVertexTangentTypeSelector<EStaticMeshVertexTangentBasisType::HighPrecision>::TangentTypeT> TangentType;
		TangentsStride = sizeof(TangentType);
		TangentsData = new TStaticMeshVertexData<TangentType>(bNeedsCPUAccess);
	}
	else
	{
		typedef TStaticMeshVertexTangentDatum<typename TStaticMeshVertexTangentTypeSelector<EStaticMeshVertexTangentBasisType::Default>::TangentTypeT> TangentType;
		TangentsStride = sizeof(TangentType);
		TangentsData = new TStaticMeshVertexData<TangentType>(bNeedsCPUAccess);
	}

	if (GetUseFullPrecisionUVs())
	{
		typedef TStaticMeshVertexUVsDatum<typename TStaticMeshVertexUVsTypeSelector<EStaticMeshVertexUVType::HighPrecision>::UVsTypeT> UVType;
		TexcoordStride = sizeof(UVType);
		TexcoordData = new TStaticMeshVertexData<UVType>(bNeedsCPUAccess);
	}
	else
	{
		typedef TStaticMeshVertexUVsDatum<typename TStaticMeshVertexUVsTypeSelector<EStaticMeshVertexUVType::Default>::UVsTypeT> UVType;
		TexcoordStride = sizeof(UVType);
		TexcoordData = new TStaticMeshVertexData<UVType>(bNeedsCPUAccess);
	}
}

int FStaticMeshVertexBuffer::GetTangentSize() const
{
	if (GetUseHighPrecisionTangentBasis())
	{
		typedef TStaticMeshVertexTangentDatum<typename TStaticMeshVertexTangentTypeSelector<EStaticMeshVertexTangentBasisType::HighPrecision>::TangentTypeT> TangentType;
		TangentsStride = sizeof(TangentType);
		return TangentsStride * GetNumVertices();
	}
	else
	{
		typedef TStaticMeshVertexTangentDatum<typename TStaticMeshVertexTangentTypeSelector<EStaticMeshVertexTangentBasisType::Default>::TangentTypeT> TangentType;
		TangentsStride = sizeof(TangentType);
		return TangentsStride * GetNumVertices();
	}
}

int FStaticMeshVertexBuffer::GetTexCoordSize() const
{
	if (GetUseFullPrecisionUVs())
	{
		typedef TStaticMeshVertexUVsDatum<typename TStaticMeshVertexUVsTypeSelector<EStaticMeshVertexUVType::HighPrecision>::UVsTypeT> UVType;
		TexcoordStride = sizeof(UVType);
		return TexcoordStride * GetNumTexCoords() * GetNumVertices();
	}
	else
	{
		typedef TStaticMeshVertexUVsDatum<typename TStaticMeshVertexUVsTypeSelector<EStaticMeshVertexUVType::Default>::UVsTypeT> UVType;
		TexcoordStride = sizeof(UVType);
		return TexcoordStride * GetNumTexCoords() * GetNumVertices();
	}
}

FShaderResourceViewRHIRef FStaticMeshVertexBuffer::GetOrCreateTangentsSRV(FRHICommandListBase& RHICmdList) const
{
	// Get the existing SRV or create a new externally owned SRV
	if (TangentsSRV.IsValid())
	{
		return TangentsSRV;
	}
	
	if (!TangentsVertexBuffer.VertexBufferRHI)
	{
		return {};
	}
	
	uint32       Stride = GetUseHighPrecisionTangentBasis() ? 8 : 4;
	EPixelFormat Format = GetUseHighPrecisionTangentBasis() ? PF_R16G16B16A16_SNORM : PF_R8G8B8A8_SNORM;
	
	FShaderResourceViewRHIRef ExternalTangentsSRV = RHICmdList.CreateShaderResourceView(
		TangentsVertexBuffer.VertexBufferRHI,
		FRHIViewDesc::CreateBufferSRV()
		.SetType(FRHIViewDesc::EBufferType::Typed)
		.SetFormat(Format));
	
	return ExternalTangentsSRV;
}

FShaderResourceViewRHIRef FStaticMeshVertexBuffer::GetOrCreateTexCoordsSRV(FRHICommandListBase& RHICmdList) const
{
	// Get the existing SRV or create a new externally owned SRV
	if (TextureCoordinatesSRV.IsValid())
	{
		return TextureCoordinatesSRV;
	}
	
	if (!TexCoordVertexBuffer.VertexBufferRHI)
	{
		return {};
	}

	uint32       Stride = GetUseFullPrecisionUVs() ? 8 : 4;
	EPixelFormat Format = GetUseFullPrecisionUVs() ? PF_G32R32F : PF_G16R16F;
	FShaderResourceViewRHIRef ExternalTextureCoordinatesSRV = RHICmdList.CreateShaderResourceView(
		TexCoordVertexBuffer.VertexBufferRHI,
		FRHIViewDesc::CreateBufferSRV()
			.SetType(FRHIViewDesc::EBufferType::Typed)
			.SetFormat(Format));

	return ExternalTextureCoordinatesSRV;
}

void FStaticMeshVertexBuffer::InitTangentAndTexCoordStrides()
{
	typedef TStaticMeshVertexTangentDatum<typename TStaticMeshVertexTangentTypeSelector<EStaticMeshVertexTangentBasisType::HighPrecision>::TangentTypeT> HighPrecTangentType;
	typedef TStaticMeshVertexTangentDatum<typename TStaticMeshVertexTangentTypeSelector<EStaticMeshVertexTangentBasisType::Default>::TangentTypeT> DefaultTangentType;
	typedef TStaticMeshVertexUVsDatum<typename TStaticMeshVertexUVsTypeSelector<EStaticMeshVertexUVType::HighPrecision>::UVsTypeT> HighPrecUVType;
	typedef TStaticMeshVertexUVsDatum<typename TStaticMeshVertexUVsTypeSelector<EStaticMeshVertexUVType::Default>::UVsTypeT> DefaultUVType;

	TangentsStride = GetUseHighPrecisionTangentBasis() ? sizeof(HighPrecTangentType) : sizeof(DefaultTangentType);
	TexcoordStride = GetUseFullPrecisionUVs() ? sizeof(HighPrecUVType) : sizeof(DefaultUVType);
}

void FStaticMeshVertexBuffer::BindTangentVertexBuffer(const FVertexFactory* VertexFactory, FStaticMeshDataType& Data) const
{
	{
		Data.TangentsSRV = TangentsSRV;
	}

	{
		uint32 TangentSizeInBytes = 0;
		uint32 TangentXOffset = 0;
		uint32 TangentZOffset = 0;
		EVertexElementType TangentElemType = VET_None;

		if (GetUseHighPrecisionTangentBasis())
		{
			typedef TStaticMeshVertexTangentDatum<typename TStaticMeshVertexTangentTypeSelector<EStaticMeshVertexTangentBasisType::HighPrecision>::TangentTypeT> TangentType;
			TangentElemType = TStaticMeshVertexTangentTypeSelector<EStaticMeshVertexTangentBasisType::HighPrecision>::VertexElementType;
			TangentXOffset = STRUCT_OFFSET(TangentType, TangentX);
			TangentZOffset = STRUCT_OFFSET(TangentType, TangentZ);
			TangentSizeInBytes = sizeof(TangentType);

		}
		else
		{
			typedef TStaticMeshVertexTangentDatum<typename TStaticMeshVertexTangentTypeSelector<EStaticMeshVertexTangentBasisType::Default>::TangentTypeT> TangentType;
			TangentElemType = TStaticMeshVertexTangentTypeSelector<EStaticMeshVertexTangentBasisType::Default>::VertexElementType;
			TangentXOffset = STRUCT_OFFSET(TangentType, TangentX);
			TangentZOffset = STRUCT_OFFSET(TangentType, TangentZ);
			TangentSizeInBytes = sizeof(TangentType);
		}

		Data.TangentBasisComponents[0] = FVertexStreamComponent(
			&TangentsVertexBuffer,
			TangentXOffset,
			TangentSizeInBytes,
			TangentElemType,
			EVertexStreamUsage::ManualFetch
		);

		Data.TangentBasisComponents[1] = FVertexStreamComponent(
			&TangentsVertexBuffer,
			TangentZOffset,
			TangentSizeInBytes,
			TangentElemType,
			EVertexStreamUsage::ManualFetch
		);
	}
}

void FStaticMeshVertexBuffer::BindPackedTexCoordVertexBuffer(const FVertexFactory* VertexFactory, FStaticMeshDataType& Data, int32 MaxNumTexCoords) const
{
	Data.TextureCoordinates.Empty();
	Data.NumTexCoords = GetNumTexCoords();

	{
		Data.TextureCoordinatesSRV = TextureCoordinatesSRV;
	}

	{
		EVertexElementType UVDoubleWideVertexElementType = VET_None;
		EVertexElementType UVVertexElementType = VET_None;
		uint32 UVSizeInBytes = 0;
		if (GetUseFullPrecisionUVs())
		{
			UVSizeInBytes = sizeof(TStaticMeshVertexUVsTypeSelector<EStaticMeshVertexUVType::HighPrecision>::UVsTypeT);
			UVDoubleWideVertexElementType = VET_Float4;
			UVVertexElementType = VET_Float2;
		}
		else
		{
			UVSizeInBytes = sizeof(TStaticMeshVertexUVsTypeSelector<EStaticMeshVertexUVType::Default>::UVsTypeT);
			UVDoubleWideVertexElementType = VET_Half4;
			UVVertexElementType = VET_Half2;
		}

		uint32 UvStride = UVSizeInBytes * GetNumTexCoords();

		// If the max num of UVs is specified, clamp to that number.
		int32 ClampedNumTexCoords = GetNumTexCoords();
		if (MaxNumTexCoords > -1)
		{
			ClampedNumTexCoords = FMath::Min<int32>(GetNumTexCoords(), MaxNumTexCoords);
		}
		check(ClampedNumTexCoords >= 0);

		int32 UVIndex;
		for (UVIndex = 0; UVIndex < (int32)ClampedNumTexCoords - 1; UVIndex += 2)
		{
			Data.TextureCoordinates.Add(FVertexStreamComponent(
				&TexCoordVertexBuffer,
				UVSizeInBytes * UVIndex,
				UvStride,
				UVDoubleWideVertexElementType,
				EVertexStreamUsage::ManualFetch
			));
		}

		// possible last UV channel if we have an odd number
		if (UVIndex < (int32)ClampedNumTexCoords)
		{
			Data.TextureCoordinates.Add(FVertexStreamComponent(
				&TexCoordVertexBuffer,
				UVSizeInBytes * UVIndex,
				UvStride,
				UVVertexElementType,
				EVertexStreamUsage::ManualFetch
			));
		}
	}
}

void FStaticMeshVertexBuffer::BindTexCoordVertexBuffer(const FVertexFactory* VertexFactory, FStaticMeshDataType& Data, int ClampedNumTexCoords) const
{
	Data.TextureCoordinates.Empty();
	Data.NumTexCoords = GetNumTexCoords();

	{
		Data.TextureCoordinatesSRV = TextureCoordinatesSRV;
	}

	{
		EVertexElementType UVVertexElementType = VET_None;
		uint32 UVSizeInBytes = 0;

		if (GetUseFullPrecisionUVs())
		{
			UVSizeInBytes = sizeof(TStaticMeshVertexUVsTypeSelector<EStaticMeshVertexUVType::HighPrecision>::UVsTypeT);
			UVVertexElementType = VET_Float2;
		}
		else
		{
			UVSizeInBytes = sizeof(TStaticMeshVertexUVsTypeSelector<EStaticMeshVertexUVType::Default>::UVsTypeT);
			UVVertexElementType = VET_Half2;
		}

		uint32 UvStride = UVSizeInBytes * GetNumTexCoords();

		if (ClampedNumTexCoords > -1)
		{
			ClampedNumTexCoords = FMath::Min<uint32>(GetNumTexCoords(), MAX_TEXCOORDS);
		}
		else
		{
			ClampedNumTexCoords = GetNumTexCoords();
		}

		check(ClampedNumTexCoords >= 0);

		for (uint32 UVIndex = 0; UVIndex < (uint32)ClampedNumTexCoords; UVIndex++)
		{
			Data.TextureCoordinates.Add(FVertexStreamComponent(
				&TexCoordVertexBuffer,
				UVSizeInBytes * UVIndex,
				UvStride,
				UVVertexElementType,
				EVertexStreamUsage::ManualFetch
			));
		}
	}
}

void FStaticMeshVertexBuffer::BindLightMapVertexBuffer(const FVertexFactory* VertexFactory, FStaticMeshDataType& Data, int LightMapCoordinateIndex) const
{
	LightMapCoordinateIndex = LightMapCoordinateIndex < (int32)GetNumTexCoords() ? LightMapCoordinateIndex : (int32)GetNumTexCoords() - 1;
	//FIXME: pso precache triggers this before mesh postload has completed. normally, EnforceLightmapRestrictions called from mesh postload prevents this
	//check(LightMapCoordinateIndex >= 0);  

	Data.LightMapCoordinateIndex = LightMapCoordinateIndex;
	Data.NumTexCoords = GetNumTexCoords();

	{
		Data.TextureCoordinatesSRV = TextureCoordinatesSRV;
	}

	{
		EVertexElementType UVVertexElementType = VET_None;
		uint32 UVSizeInBytes = 0;

		if (GetUseFullPrecisionUVs())
		{
			UVSizeInBytes = sizeof(TStaticMeshVertexUVsTypeSelector<EStaticMeshVertexUVType::HighPrecision>::UVsTypeT);
			UVVertexElementType = VET_Float2;
		}
		else
		{
			UVSizeInBytes = sizeof(TStaticMeshVertexUVsTypeSelector<EStaticMeshVertexUVType::Default>::UVsTypeT);
			UVVertexElementType = VET_Half2;
		}

		uint32 UvStride = UVSizeInBytes * GetNumTexCoords();

		if (LightMapCoordinateIndex >= 0 && (uint32)LightMapCoordinateIndex < GetNumTexCoords())
		{
			Data.LightMapCoordinateComponent = FVertexStreamComponent(
				&TexCoordVertexBuffer,
				UVSizeInBytes * LightMapCoordinateIndex,
				UvStride,
				UVVertexElementType, 
				EVertexStreamUsage::ManualFetch
			);
		}
	}
}
