// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	Texture2DResource.cpp: Implementation of FTexture2DResource.
=============================================================================*/

#include "Rendering/StreamableTextureResource.h"
#include "DeviceProfiles/DeviceProfile.h"
#include "DeviceProfiles/DeviceProfileManager.h"
#include "Misc/CoreStats.h"
#include "ProfilingDebugging/LoadTimeTracker.h"
#include "ProfilingDebugging/ScopedDebugInfo.h"
#include "Stats/StatsTrace.h"
#include "RHIUtilities.h"
#include "Engine/TextureAllMipDataProviderFactory.h"

#if STATS
TAtomic<int64> GUITextureMemory = 0;
TAtomic<int64> GNeverStreamTextureMemory = 0;
#endif

static TAutoConsoleVariable<int32> CVarPartiallyResidentTextureEnabled(
	TEXT("r.PRT.Enable"),
	1,
	TEXT("If set to 1, textures will use virtual memory on supported platforms so that they can be partially resident."),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarPartiallyResidentTextureReducedMemory(
	TEXT("r.PRT.ReducedMemory"),
	0,
	TEXT("If set to 1, the cost of partially resident textures will be reduced by using a more packed layout."),
	ECVF_RenderThreadSafe);

static FAutoConsoleVariableDeprecated CVarPartiallyResidentTextureEnabled_Deprecated(TEXT("r.VirtualTexture"), TEXT("r.PRT.Enable"), TEXT("5.6"));
static FAutoConsoleVariableDeprecated CVarPartiallyResidentTextureReducedMemory_Deprecated(TEXT("r.VirtualTextureReducedMemory"), TEXT("r.PRT.ReducedMemory"), TEXT("5.6"));


bool CanCreateWithPartiallyResidentMips(ETextureCreateFlags TexCreateFlags)
{
#if PLATFORM_SUPPORTS_VIRTUAL_TEXTURES
	const ETextureCreateFlags iDisableFlags =
		TexCreate_RenderTargetable |
		TexCreate_ResolveTargetable |
		TexCreate_DepthStencilTargetable |
		TexCreate_Dynamic |
		TexCreate_UAV |
		TexCreate_Presentable;
	const ETextureCreateFlags iRequiredFlags =
		TexCreate_OfflineProcessed;

	return ((TexCreateFlags & (iDisableFlags | iRequiredFlags)) == iRequiredFlags) && CVarPartiallyResidentTextureEnabled.GetValueOnAnyThread();
	
#else
	return false;
#endif
}

static bool ShouldAllowPlatformTiling(UTexture* Texture)
{
	UTextureAllMipDataProviderFactory* AllMipProvider = nullptr;
	const TArray<UAssetUserData*>* UserDataArray = Texture->GetAssetUserDataArray();

	bool bTextureDataIsTiled = !Texture->bNotOfflineProcessed;
	if (UTextureMipDataProviderFactory* MipProvider = Texture->GetAssetUserData<UTextureMipDataProviderFactory>())
	{
		// If an UTextureAllMipDataProviderFactory is provided, the original texture data is unused, and all
		// texture data will come from the provider, and should always determine if the data is offline processed
		// or not.
		bool ProviderDataIsTiled = MipProvider->ShouldAllowPlatformTiling(Texture);
		if (MipProvider->IsA<UTextureAllMipDataProviderFactory>())
		{
			return ProviderDataIsTiled;
		}
		ensureMsgf(ProviderDataIsTiled == bTextureDataIsTiled, TEXT("A mip provider must provide texture data in the same layout as the texture itself."));
	}
	
	return bTextureDataIsTiled;
}

/** Scoped debug info that provides the texture name to memory allocation and crash callstacks. */
class FStreamableTextureScopedDebugInfo : public FScopedDebugInfo
{
public:

	/** Initialization constructor. */
	FStreamableTextureScopedDebugInfo(const FStreamableTextureResource* InResource):
		FScopedDebugInfo(0),
		Resource(InResource)
	{}

	// FScopedDebugInfo interface.
	virtual FString GetFunctionName() const
	{
		return FString::Printf(
			TEXT("%s (%ux%ux%u %s, %u mips, LODGroup=%u)"),
			*Resource->GetTextureName().ToString(),
			Resource->GetSizeX(),
			Resource->GetSizeY(),
			Resource->GetSizeZ(),
			GPixelFormats[Resource->GetPixelFormat()].Name,
			Resource->GetState().MaxNumLODs,
			(int32)Resource->GetLODGroup()
			);
	}
	virtual FString GetFilename() const
	{
		return FString::Printf(
			TEXT("%s../../Development/Src/Engine/%s"),
			FPlatformProcess::BaseDir(),
			ANSI_TO_TCHAR(__FILE__)
			);
	}
	virtual int32 GetLineNumber() const
	{
		return __LINE__;
	}

private:

	const FStreamableTextureResource* Resource = nullptr;
};

FStreamableTextureResource::FStreamableTextureResource(UTexture* InOwner, const FTexturePlatformData* InPlatformData, const FStreamableRenderResourceState& InPostInitState, bool bAllowPartiallyResidentMips)
	: PlatformData(InPlatformData)
	, State(InPostInitState)
	, TextureName(InOwner->GetFName())
	, LODGroup(InOwner->LODGroup)
	, PixelFormat(InPlatformData->PixelFormat)
{
	bSRGB = InOwner->SRGB;
	bGreyScaleFormat = UE::TextureDefines::ShouldUseGreyScaleEditorVisualization( InOwner->CompressionSettings );

	const UTextureLODSettings* TextureLODSettings = UDeviceProfileManager::Get().GetActiveProfile()->GetTextureLODSettings();

	Filter = (ESamplerFilter)TextureLODSettings->GetSamplerFilter(InOwner);
	AddressU = InOwner->GetTextureAddressX() == TA_Wrap ? AM_Wrap : (InOwner->GetTextureAddressX() == TA_Clamp ? AM_Clamp : AM_Mirror);
	AddressV = InOwner->GetTextureAddressY() == TA_Wrap ? AM_Wrap : (InOwner->GetTextureAddressY() == TA_Clamp ? AM_Clamp : AM_Mirror);
	AddressW = InOwner->GetTextureAddressZ() == TA_Wrap ? AM_Wrap : (InOwner->GetTextureAddressZ() == TA_Clamp ? AM_Clamp : AM_Mirror);
	MaxAniso = TextureLODSettings->GetTextureLODGroup(InOwner->LODGroup).MaxAniso;

	// Get the biggest mips size, might be different from the actual resolution (depending on NumOfResidentLODs).
	const FTexture2DMipMap& Mip0 = PlatformData->Mips[State.AssetLODBias];
	SizeX = Mip0.SizeX;	
	SizeY = Mip0.SizeY;
	SizeZ = Mip0.SizeZ;

	CreationFlags = 
		  (InOwner->SRGB ? TexCreate_SRGB : TexCreate_None)
		| (ShouldAllowPlatformTiling(InOwner) ? TexCreate_OfflineProcessed : TexCreate_None)
		| TexCreate_ShaderResource 
		| (InOwner->bNoTiling ? TexCreate_NoTiling : TexCreate_None);

	if (InPostInitState.MaxNumLODs > 1)
	{
		CreationFlags |= TexCreate_Streamable;
	}

	bUsePartiallyResidentMips = bAllowPartiallyResidentMips && InPostInitState.bSupportsStreaming && CanCreateWithPartiallyResidentMips(CreationFlags);

	STAT(LODGroupStatName = TextureGroupStatFNames[LODGroup]);
	STAT(bIsNeverStream = InOwner->NeverStream);
}

#if STATS

void FStreamableTextureResource::CalcRequestedMipsSize() 
{ 
	TextureSize = GetPlatformMipsSize(State.NumRequestedLODs); 
}

void FStreamableTextureResource::IncrementTextureStats() const
{
	INC_DWORD_STAT_BY( STAT_TextureMemory, TextureSize );
	INC_DWORD_STAT_FNAME_BY( LODGroupStatName, TextureSize );

	if (LODGroup == TEXTUREGROUP_UI)
	{
		GUITextureMemory += static_cast<int64>(TextureSize);
	}
	else if (bIsNeverStream)
	{
		GNeverStreamTextureMemory += static_cast<int64>(TextureSize);
	}
}

void FStreamableTextureResource::DecrementTextureStats() const
{
	DEC_DWORD_STAT_BY( STAT_TextureMemory, TextureSize );
	DEC_DWORD_STAT_FNAME_BY( LODGroupStatName, TextureSize );

	if (LODGroup == TEXTUREGROUP_UI)
	{
		GUITextureMemory -= static_cast<int64>(TextureSize);
	}
	else if (bIsNeverStream)
	{
		GNeverStreamTextureMemory -= static_cast<int64>(TextureSize);
	}
}
#endif

void FStreamableTextureResource::InitRHI(FRHICommandListBase& RHICmdList)
{
	SCOPED_LOADTIMER(FStreamableTextureResource_InitRHI);

	FStreamableTextureScopedDebugInfo ScopedDebugInfo(this);

	STAT(CalcRequestedMipsSize());
	STAT(IncrementTextureStats());

	RefreshSamplerStates();
	
	// Check if this is the initial creation of the texture, or if we're recreating a texture that was released by ReleaseRHI.
	static TConsoleVariableData<int32>* CVarReducedMode = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.PRT.ReducedMemory"));
	check(CVarReducedMode);
	if (bUsePartiallyResidentMips && (!CVarReducedMode->GetValueOnRenderThread() || State.NumRequestedLODs > State.NumNonStreamingLODs))
	{
		CreatePartiallyResidentTexture(RHICmdList);
	}
	else
	{
		CreateTexture(RHICmdList);
	}

	if (ensure(TextureReferenceRHI.IsValid()))
	{
		RHICmdList.UpdateTextureReference(TextureReferenceRHI, TextureRHI);
	}
}

void FStreamableTextureResource::ReleaseRHI()
{
	STAT(DecrementTextureStats());

	if (ensure(TextureReferenceRHI.IsValid()))
	{
		RHIClearTextureReference(TextureReferenceRHI);
	}

	TextureRHI.SafeRelease();
	FTextureResource::ReleaseRHI();
}

void FStreamableTextureResource::FinalizeStreaming(FRHICommandListBase& RHICmdList, FRHITexture* InTextureRHI)
{
	checkSlow(IsInRenderingThread() && InTextureRHI);

	// The new mip count must match the streaming request.
	State.NumRequestedLODs = InTextureRHI->GetNumMips();

	if (State.NumResidentLODs != State.NumRequestedLODs)
	{
		STAT(DecrementTextureStats());
		STAT(CalcRequestedMipsSize());
		STAT(IncrementTextureStats());
	}

	TextureRHI = InTextureRHI;
	TextureRHI->SetOwnerName(GetOwnerName());
	if (ensure(TextureReferenceRHI.IsValid()))
	{
		RHICmdList.UpdateTextureReference(TextureReferenceRHI, TextureRHI);
	}
	State.NumResidentLODs = State.NumRequestedLODs;
}

void FStreamableTextureResource::FinalizeStreaming(FRHITexture* InTextureRHI)
{
	FinalizeStreaming(FRHICommandListImmediate::Get(), InTextureRHI);
}

// Recreate the sampler states (used when updating mip map lod bias offset)
void FStreamableTextureResource::RefreshSamplerStates()
{
	// Create the sampler state RHI resource.
	FSamplerStateInitializerRHI SamplerStateInitializer
	(
		Filter,
		AddressU,
		AddressV,
		AddressW,
		MipBias,
		ComputeAnisotropyRT(MaxAniso) // Need this in case anisotropy is changed at runtime
	);
	SamplerStateRHI = GetOrCreateSamplerState(SamplerStateInitializer);

	// Create a custom sampler state for using this texture in a deferred pass, where ddx / ddy are discontinuous
	FSamplerStateInitializerRHI DeferredPassSamplerStateInitializer
	(
		Filter,
		AddressU,
		AddressV,
		AddressW,
		MipBias,
		// Disable anisotropic filtering, since aniso doesn't respect MaxLOD
		1,
		0,
		// Prevent the less detailed mip levels from being used, which hides artifacts on silhouettes due to ddx / ddy being very large
		// This has the side effect that it increases minification aliasing on light functions
		2
	);

	DeferredPassSamplerStateRHI = GetOrCreateSamplerState(DeferredPassSamplerStateInitializer);
}

TArrayView<const FTexture2DMipMap*> FStreamableTextureResource::GetPlatformMipsView() const
{
	return TArrayView<const FTexture2DMipMap*>(PlatformData->Mips.GetData() + State.AssetLODBias, State.MaxNumLODs);
}

const FTexture2DMipMap* FStreamableTextureResource::GetPlatformMip(int32 MipIdx) const
{
	return &PlatformData->Mips[State.AssetLODBias + MipIdx];
}


