// Copyright Epic Games, Inc. All Rights Reserved.

#include "Engine/SubsurfaceProfile.h"
#include "Engine/SubsurfaceScatteringMath.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "Math/Float16.h"
#include "Rendering/BurleyNormalizedSSS.h"
#include "EngineModule.h"
#include "RenderTargetPool.h"
#include "PixelShaderUtils.h"
#include "RenderingThread.h"
#include "UObject/FortniteMainBranchObjectVersion.h"
#include "Serialization/CustomVersion.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SubsurfaceProfile)

DEFINE_LOG_CATEGORY_STATIC(LogSubsurfaceProfile, Log, All);

static TAutoConsoleVariable<int32> CVarSSProfilesPreIntegratedTextureResolution(
	TEXT("r.SSProfilesPreIntegratedTextureResolution"),
	64,
	TEXT("The resolution of the subsurface profile preintegrated texture.\n"),
	ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<int32> CVarSSProfilesSamplingChannelSelection(
	TEXT("r.SSProfilesSamplingChannelSelection"),
	1,
	TEXT("0. Select the sampling channel based on max DMFP.\n")
	TEXT("1. based on max MFP."),
	ECVF_RenderThreadSafe
);

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
static TAutoConsoleVariable<int32> CVarSSProfilesPreIntegratedTextureForceUpdate(
	TEXT("r.SSProfilesPreIntegratedTextureForceUpdate"),
	0,
	TEXT("0: Only update the preintegrated texture as needed.\n")
	TEXT("1: Force to update the preintegrated texture for debugging.\n"),
	ECVF_Cheat | ECVF_RenderThreadSafe);
#endif

static bool ForceUpdateSSProfilesPreIntegratedTexture()
{
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	return (CVarSSProfilesPreIntegratedTextureForceUpdate.GetValueOnAnyThread() == 1);
#else
	return false;
#endif
}

// lives on the render thread
ENGINE_API TGlobalResource<FSubsurfaceProfileTexture> GSubsurfaceProfileTextureObject;

// Texture with one or more SubSurfaceProfiles or 0 if there is no user
static TRefCountPtr<IPooledRenderTarget> GSSProfiles;

// Texture with one or more pre-integrated textures or 0 if there is no user
static TRefCountPtr<IPooledRenderTarget> GSSProfilesPreIntegratedTexture;


// Custom serialization for USubsurfaceProfile
struct FSubsurfaceProfileCustomVersion
{
	enum Type
	{
		// Before any version changes were made
		BeforeCustomVersionWasAdded = 0,

		// Old profile will change extinction value and transmission unit scale
		ExtinctionTransmissionMultiplierFix,

		// Old profile will move one extra surface albedo to transmission tint.
		FactorExtraSurfaceAlbedoToTransmissionTintFix,

		// -----<new versions can be added above this line>-------------------------------------------------
		VersionPlusOne,
		LatestVersion = VersionPlusOne - 1
	};

	// The GUID for this custom version number
	const static FGuid GUID;

private:
	FSubsurfaceProfileCustomVersion() {}
};

const FGuid FSubsurfaceProfileCustomVersion::GUID(0xDCC215F2, 0x6CB44CD0, 0x9693A2A0, 0xBE6A6D23);

// Register the custom version with core
FCustomVersionRegistration GRegisterSubsurfaceProfileCustomVersion(FSubsurfaceProfileCustomVersion::GUID, FSubsurfaceProfileCustomVersion::LatestVersion, TEXT("SubsurfaceProfileVer"));


void ConvertSubsurfaceParametersFromSeparableToBurley(const FSubsurfaceProfileStruct& Settings, FLinearColor& SurfaceAlbedo, FLinearColor& MeanFreePathColor, float& MeanFreePathDistance)
{
	MapFallOffColor2SurfaceAlbedoAndDiffuseMeanFreePath(Settings.FalloffColor.R, SurfaceAlbedo.R, MeanFreePathColor.R);
	MapFallOffColor2SurfaceAlbedoAndDiffuseMeanFreePath(Settings.FalloffColor.G, SurfaceAlbedo.G, MeanFreePathColor.G);
	MapFallOffColor2SurfaceAlbedoAndDiffuseMeanFreePath(Settings.FalloffColor.B, SurfaceAlbedo.B, MeanFreePathColor.B);

	//Normalize mean free path color and set the corresponding dfmp
	float MaxMeanFreePathColor = FMath::Max3(MeanFreePathColor.R, MeanFreePathColor.G, MeanFreePathColor.B);
	if (MaxMeanFreePathColor > 1)
	{
		MeanFreePathColor /= MaxMeanFreePathColor;
		MeanFreePathDistance = FMath::Clamp(Settings.ScatterRadius * MaxMeanFreePathColor, 0.1f, 50.0f);	// 50.0f is the ClampMax of MeanFreePathDistance.
	}
}

void UpgradeSeparableToBurley(FSubsurfaceProfileStruct& Settings)
{
	ConvertSubsurfaceParametersFromSeparableToBurley(Settings, Settings.SurfaceAlbedo, Settings.MeanFreePathColor, Settings.MeanFreePathDistance);
	Settings.MeanFreePathDistance /= (2.229f * 2.229f);
	Settings.Tint = Settings.SubsurfaceColor;
	Settings.bEnableBurley = true;
}

void UpgradeDiffuseMeanFreePathToMeanFreePath(FSubsurfaceProfileStruct& Settings)
{
	// 1. Update dmfp to mean free path
	const float CmToMm = 10.0f;
	
	FLinearColor OldDmfp = Settings.MeanFreePathColor * Settings.MeanFreePathDistance;
	
	FLinearColor Mfp = SubsurfaceScattering::BurleyNormalized::GetMeanFreePathFromDiffuseMeanFreePath(Settings.SurfaceAlbedo, OldDmfp);

	Settings.MeanFreePathDistance = FMath::Max3(Mfp.R, Mfp.G, Mfp.B);
	Settings.MeanFreePathColor = Mfp / Settings.MeanFreePathDistance;

	// Support mfp < 0.1f
	if (Settings.MeanFreePathDistance < 0.1f)
	{
		Settings.MeanFreePathColor = Settings.MeanFreePathColor * (Settings.MeanFreePathDistance / 0.1f);
		Settings.MeanFreePathDistance = 0.1f;
	}

	Settings.bEnableMeanFreePath = true;

	//2. Fix scaling
	// Previously, the scaling is scaled up by 1/(SUBSURFACE_KERNEL_SIZE / BURLEY_CM_2_MM). To maintain the same
	// visual appearance, apply that scale up to world unit scale
	Settings.WorldUnitScale /= (SUBSURFACE_KERNEL_SIZE / CmToMm);
}

void UpgradeSubsurfaceProfileParameters(FSubsurfaceProfileStruct& Settings)
{
	if (!Settings.bEnableBurley)
	{
		UpgradeSeparableToBurley(Settings);
	}

	if (!Settings.bEnableMeanFreePath)
	{
		UpgradeDiffuseMeanFreePathToMeanFreePath(Settings);
	}
}

FSubsurfaceProfileTexture::FSubsurfaceProfileTexture()
{
	check(IsInGameThread());

	FSubsurfaceProfileStruct DefaultSkin;

	//The default burley in slot 0 behaves the same as Separable previously
	UpgradeSubsurfaceProfileParameters(DefaultSkin);

	// Default skin is not loaded from the disk, so we have to add this so that the default retains the old look for backward compatability.
	// Same to Ar.CustomVer(FSubsurfaceProfileCustomVersion::GUID) < FSubsurfaceProfileCustomVersion::ExtinctionTransmissionMultiplierFix
	{
		DefaultSkin.ExtinctionScale *= 3.1f;

		// The transmission distance was scaled up by 10.0 in old profiles.
		DefaultSkin.DistanceScale *= 10.0f;
	}

	// Same to Ar.CustomVer(FSubsurfaceProfileCustomVersion::GUID) < FSubsurfaceProfileCustomVersion::FactorExtraSurfaceAlbedoToTransmissionTintFix
	{
		DefaultSkin.TransmissionTintColor *= DefaultSkin.SurfaceAlbedo;
	}

	// add element 0, it is used as default profile
	SubsurfaceProfileEntries.Add(FSubsurfaceProfileEntry(DefaultSkin, 0));
}

FSubsurfaceProfileTexture::~FSubsurfaceProfileTexture()
{
}

int32 FSubsurfaceProfileTexture::AddProfile(const FSubsurfaceProfileStruct Settings, const USubsurfaceProfile* InProfile)
{
	check(InProfile);
	check(FindAllocationId(InProfile) == -1);

	int32 RetAllocationId = -1;
	{
		for (int32 i = 1; i < SubsurfaceProfileEntries.Num(); ++i)
		{
			if (SubsurfaceProfileEntries[i].Profile == 0)
			{
				RetAllocationId = i;
				SubsurfaceProfileEntries[RetAllocationId].Profile = InProfile;
				break;
			}
		}

		if(RetAllocationId == -1)
		{
			RetAllocationId = SubsurfaceProfileEntries.Num();
			SubsurfaceProfileEntries.Add(FSubsurfaceProfileEntry(Settings, InProfile));
		}
	}

	UpdateProfile(RetAllocationId, Settings);

	return RetAllocationId;
}


void FSubsurfaceProfileTexture::RemoveProfile(const USubsurfaceProfile* InProfile)
{
	int32 AllocationId = FindAllocationId(InProfile);

	if(AllocationId == -1)
	{
		// -1: no allocation, no work needed
		return;
	}

	// >0 as 0 is used as default profile which should never be removed
	check(AllocationId > 0);

	check(SubsurfaceProfileEntries[AllocationId].Profile == InProfile);

	// make it available for reuse
	SubsurfaceProfileEntries[AllocationId].Profile = 0;
	SubsurfaceProfileEntries[AllocationId].Settings.Invalidate();
}

void FSubsurfaceProfileTexture::UpdateProfile(int32 AllocationId, const FSubsurfaceProfileStruct Settings)
{
	check(IsInRenderingThread());


	if (AllocationId == -1)
	{
		// if we modify a profile that is not assigned/used yet, no work is needed
		return;
	}

	check(AllocationId < SubsurfaceProfileEntries.Num());

	SubsurfaceProfileEntries[AllocationId].Settings = Settings;

	GSSProfiles.SafeRelease();
	GSSProfilesPreIntegratedTexture.SafeRelease();
}

IPooledRenderTarget* FSubsurfaceProfileTexture::GetTexture()
{
	return GSSProfiles;
}

IPooledRenderTarget* FSubsurfaceProfileTexture::GetTexture(FRHICommandListImmediate& RHICmdList)
{
	if (!GSSProfiles)
	{
		CreateTexture(RHICmdList);
	}

	return GSSProfiles;
}

class FSSProfilePreIntegratedPS : public FGlobalShader
{
public:
	DECLARE_GLOBAL_SHADER(FSSProfilePreIntegratedPS);
	SHADER_USE_PARAMETER_STRUCT(FSSProfilePreIntegratedPS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_TEXTURE(Texture2D, SourceSSProfilesTexture)
		SHADER_PARAMETER_SAMPLER(SamplerState, SourceSSProfilesSampler)
		SHADER_PARAMETER(FVector4f, SourceSSProfilesTextureSizeAndInvSize)
		SHADER_PARAMETER(FVector4f, TargetSSProfilesPreIntegratedTextureSizeAndInvSize)
		SHADER_PARAMETER(int32, SourceSubsurfaceProfileInt)
		RENDER_TARGET_BINDING_SLOTS()
	END_SHADER_PARAMETER_STRUCT()

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return IsMobilePlatform(Parameters.Platform);
	}
};

IMPLEMENT_GLOBAL_SHADER(FSSProfilePreIntegratedPS, "/Engine/Private/SSProfilePreIntegratedMobile.usf", "SSProfilePreIntegratedPS", SF_Pixel);

IPooledRenderTarget* FSubsurfaceProfileTexture::GetSSProfilesPreIntegratedTexture(FRDGBuilder& GraphBuilder, EShaderPlatform ShaderPlatform)
{
	// For now the pre-integrated texture only used on mobile platform
	if (!IsMobilePlatform(ShaderPlatform))
	{
		return nullptr;
	}

	// PreIntegrated SSS look up texture
	int32 SSProfilesPreIntegratedTextureResolution = FMath::RoundUpToPowerOfTwo(FMath::Max(CVarSSProfilesPreIntegratedTextureResolution.GetValueOnAnyThread(), 32));
	
	// Generate the new preintegrated texture if needed.
	if (!GSSProfilesPreIntegratedTexture ||
		GSSProfilesPreIntegratedTexture->GetDesc().Extent != SSProfilesPreIntegratedTextureResolution ||
		ForceUpdateSSProfilesPreIntegratedTexture())
	{
		GSSProfilesPreIntegratedTexture.SafeRelease();

		int32 SubsurfaceProfileEntriesNum = SubsurfaceProfileEntries.Num();
		// Use RGB10A2 since it could be compressed by mobile hardware according to ARM.
		FRDGTextureDesc ProfileTextureDesc = FRDGTextureDesc::Create2DArray(
			SSProfilesPreIntegratedTextureResolution, 
			PF_A2B10G10R10, 
			FClearValueBinding::Black, 
			TexCreate_TargetArraySlicesIndependently | TexCreate_ShaderResource | TexCreate_RenderTargetable,
			SubsurfaceProfileEntriesNum);
		FRDGTextureRef ProfileTexture = GraphBuilder.CreateTexture(ProfileTextureDesc, TEXT("SSProfilePreIntegratedTexture"));

		for (int32 i = 0; i < SubsurfaceProfileEntriesNum; ++i)
		{
			FSSProfilePreIntegratedPS::FParameters* PassParameters = GraphBuilder.AllocParameters<FSSProfilePreIntegratedPS::FParameters>();
			PassParameters->RenderTargets[0] = FRenderTargetBinding(ProfileTexture, ERenderTargetLoadAction::EClear, 0, i);

			PassParameters->SourceSSProfilesTexture = SubsurfaceProfile::GetSubsurfaceProfileTextureWithFallback();
			PassParameters->SourceSSProfilesSampler = TStaticSamplerState<SF_Point, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();

			FIntVector SourceSSProfilesTextureSize = PassParameters->SourceSSProfilesTexture->GetSizeXYZ();

			PassParameters->SourceSSProfilesTextureSizeAndInvSize = FVector4f(SourceSSProfilesTextureSize.X, SourceSSProfilesTextureSize.Y, 1.0f / SourceSSProfilesTextureSize.X, 1.0f / SourceSSProfilesTextureSize.Y);
			PassParameters->TargetSSProfilesPreIntegratedTextureSizeAndInvSize = FVector4f(SSProfilesPreIntegratedTextureResolution, SSProfilesPreIntegratedTextureResolution, 1.0f / SSProfilesPreIntegratedTextureResolution, 1.0f / SSProfilesPreIntegratedTextureResolution);
			PassParameters->SourceSubsurfaceProfileInt = i;

			FIntRect ViewRect = FIntRect(FIntPoint(0, 0), SSProfilesPreIntegratedTextureResolution);

			const auto GlobalShaderMap = GetGlobalShaderMap(ERHIFeatureLevel::ES3_1);
			TShaderMapRef<FSSProfilePreIntegratedPS> PixelShader(GlobalShaderMap);

			FPixelShaderUtils::AddFullscreenPass(GraphBuilder, GlobalShaderMap, RDG_EVENT_NAME("SSS::SSProfilePreIntegrated"), PixelShader, PassParameters, ViewRect);
		}

		GSSProfilesPreIntegratedTexture = ConvertToExternalAccessTexture(GraphBuilder, ProfileTexture);
	}

	return GSSProfilesPreIntegratedTexture;
}

void FSubsurfaceProfileTexture::ReleaseRHI()
{
	GSSProfiles.SafeRelease();
	GSSProfilesPreIntegratedTexture.SafeRelease();
}

static float GetNextSmallerPositiveFloat(float x)
{
	check(x > 0);
	uint32 bx = *(uint32 *)&x;

	// float are ordered like int, at least for the positive part
	uint32 ax = bx - 1;

	return *(float *)&ax;
}

// NOTE: Changing offsets below requires updating all instances of #SSSS_CONSTANTS
// TODO: This needs to be defined in a single place and shared between C++ and shaders!
#define SSSS_TINT_SCALE_OFFSET					0
#define BSSS_SURFACEALBEDO_OFFSET               (SSSS_TINT_SCALE_OFFSET+1)
#define BSSS_DMFP_OFFSET                        (BSSS_SURFACEALBEDO_OFFSET+1)
#define SSSS_TRANSMISSION_OFFSET				(BSSS_DMFP_OFFSET+1)
#define SSSS_BOUNDARY_COLOR_BLEED_OFFSET		(SSSS_TRANSMISSION_OFFSET+1)
#define SSSS_DUAL_SPECULAR_OFFSET				(SSSS_BOUNDARY_COLOR_BLEED_OFFSET+1)
#define SSSS_KERNEL0_OFFSET						(SSSS_DUAL_SPECULAR_OFFSET+1)
#define SSSS_KERNEL0_SIZE						13
#define SSSS_KERNEL1_OFFSET						(SSSS_KERNEL0_OFFSET + SSSS_KERNEL0_SIZE)
#define SSSS_KERNEL1_SIZE						9
#define SSSS_KERNEL2_OFFSET						(SSSS_KERNEL1_OFFSET + SSSS_KERNEL1_SIZE)
#define SSSS_KERNEL2_SIZE						6
#define SSSS_KERNEL_TOTAL_SIZE					(SSSS_KERNEL0_SIZE + SSSS_KERNEL1_SIZE + SSSS_KERNEL2_SIZE)
#define BSSS_TRANSMISSION_PROFILE_OFFSET		(SSSS_KERNEL0_OFFSET + SSSS_KERNEL_TOTAL_SIZE)
#define BSSS_TRANSMISSION_PROFILE_SIZE			32
#define	SSSS_MAX_TRANSMISSION_PROFILE_DISTANCE	5.0f // See MaxTransmissionProfileDistance in ComputeTransmissionProfile(), SeparableSSS.cpp
#define	SSSS_MAX_DUAL_SPECULAR_ROUGHNESS		2.0f

//------------------------------------------------------------------------------------------
// Consistent in BurleyNormalizedSSSCommon.ush and SubsurfaceProfile.cpp

#define SSS_TYPE_BURLEY	    0
#define SSS_TYPE_SSSS		1

// Make sure UIMax|ClampMax of WorldUnitScale * ENC_WORLDUNITSCALE_IN_CM_TO_UNIT <= 1
#define ENC_WORLDUNITSCALE_IN_CM_TO_UNIT 0.02f
#define DEC_UNIT_TO_WORLDUNITSCALE_IN_CM 1/ENC_WORLDUNITSCALE_IN_CM_TO_UNIT

// Make sure DiffuseMeanFreePath * 10(cm to mm) * ENC_DIFFUSEMEANFREEPATH_IN_MM_TO_UNIT <= 1
// Although UI switches to mean free path, the diffuse mean free path is maintained to have the 
// same range as before [0, 50] cm.
// 1 mfp can map to 1.44 dmfp (surface albedo -> 0.0) or 43.50 dmfp (surface albedo -> 1.0).
#define ENC_DIFFUSEMEANFREEPATH_IN_MM_TO_UNIT (0.01f*0.2f)
#define DEC_UNIT_TO_DIFFUSEMEANFREEPATH_IN_MM 1/ENC_DIFFUSEMEANFREEPATH_IN_MM_TO_UNIT

#define ENC_EXTINCTIONSCALE_FACTOR	0.01f
#define DEC_EXTINCTIONSCALE_FACTOR  1/ENC_EXTINCTIONSCALE_FACTOR
//------------------------------------------------------------------------------------------

//in [0,1]
float EncodeWorldUnitScale(float WorldUnitScale)
{
	return WorldUnitScale * ENC_WORLDUNITSCALE_IN_CM_TO_UNIT;
}

float DecodeWorldUnitScale(float EncodedWorldUnitScale)
{
	return EncodedWorldUnitScale * DEC_UNIT_TO_WORLDUNITSCALE_IN_CM;
}

//in [0,1]
FLinearColor EncodeDiffuseMeanFreePath(FLinearColor DiffuseMeanFreePath)
{
	return DiffuseMeanFreePath * ENC_DIFFUSEMEANFREEPATH_IN_MM_TO_UNIT;
}

FLinearColor DecodeDiffuseMeanFreePath(FLinearColor EncodedDiffuseMeanFreePath)
{
	return EncodedDiffuseMeanFreePath * DEC_UNIT_TO_DIFFUSEMEANFREEPATH_IN_MM;
}

//in [0,1]
float EncodeScatteringDistribution(float ScatteringDistribution)
{
	return (ScatteringDistribution + 1.0f) * 0.5f;
}

float DecodeScatteringDistribution(float ScatteringDistribution)
{
	return ScatteringDistribution * 2.0f - 1.0f;
}

float EncodeExtinctionScale(float ExtinctionScale)
{
	return ExtinctionScale * ENC_EXTINCTIONSCALE_FACTOR;
}

float DecodeExtinctionScale(float ExtinctionScale)
{
	return ExtinctionScale * DEC_EXTINCTIONSCALE_FACTOR;
}


void SetupSurfaceAlbedoAndDiffuseMeanFreePath(FLinearColor& SurfaceAlbedo, FLinearColor& Dmfp)
{
	int32 SamplingSelectionMethod = FMath::Clamp(CVarSSProfilesSamplingChannelSelection.GetValueOnAnyThread(), 0, 1);
	FLinearColor Distance = SamplingSelectionMethod == 0 ?
		Dmfp															// 0: by max diffuse mean free path
		: SubsurfaceScattering::BurleyNormalized::GetMeanFreePathFromDiffuseMeanFreePath(SurfaceAlbedo, Dmfp);	// 1: by max mean free path
	//Store the value that corresponds to the largest Dmfp (diffuse mean free path) channel to A channel.
	//This is an optimization to shift finding the max correspondence workload
	//to CPU.
	const float MaxComp = FMath::Max3(Distance.R, Distance.G, Distance.B);
	const uint32 IndexOfMaxComp = (Distance.R == MaxComp) ? 0 : ((Distance.G == MaxComp) ? 1 : 2);

	SurfaceAlbedo.A = SurfaceAlbedo.Component(IndexOfMaxComp);
	Dmfp.A = Dmfp.Component(IndexOfMaxComp);

	// Apply clamping so that dmfp is within encoding range.
	Dmfp = Dmfp.GetClamped(0.0f, DEC_UNIT_TO_DIFFUSEMEANFREEPATH_IN_MM);
}

float Sqrt2(float X)
{
	return sqrtf(sqrtf(X));
}

float Pow4(float X)
{
	return X * X * X * X;
}

void FSubsurfaceProfileTexture::CreateTexture(FRHICommandListImmediate& RHICmdList)
{
	uint32 Height = SubsurfaceProfileEntries.Num();

	check(Height);

	// true:16bit (currently required to have very small and very large kernel sizes), false: 8bit
	const bool b16Bit = true;

	// Each row of the texture contains SSS parameters, followed by 3 precomputed kernels. Texture must be wide enough to fit all data.
	const uint32 Width = BSSS_TRANSMISSION_PROFILE_OFFSET + BSSS_TRANSMISSION_PROFILE_SIZE;

	// at minimum 64 lines (less reallocations)
	FPooledRenderTargetDesc Desc(FPooledRenderTargetDesc::Create2DDesc(FIntPoint(Width, FMath::Max(Height, (uint32)64)), PF_B8G8R8A8, FClearValueBinding::None, TexCreate_None, TexCreate_ShaderResource, false));
	if (b16Bit)
	{
		Desc.Format = PF_A16B16G16R16;
	}
	
	GRenderTargetPool.FindFreeElement(RHICmdList, Desc, GSSProfiles, TEXT("SSProfiles"));

	// Write the contents of the texture.
	uint32 DestStride;
	uint8* DestBuffer = (uint8*)RHICmdList.LockTexture2D(GSSProfiles->GetRHI(), 0, RLM_WriteOnly, DestStride, false);

	FLinearColor TextureRow[Width];
	FMemory::Memzero(TextureRow);

	// bias to avoid div by 0 and a jump to a different value
	// this basically means we don't want subsurface scattering
	// 0.0001f turned out to be too small to fix the issue (for a small KernelSize)
	const float Bias = 0.009f;
	const float CmToMm = 10.f;
	const float MmToCm = 0.1f;
	const float FloatScaleInitial = 0x10000;
	const float FloatScale = GetNextSmallerPositiveFloat(FloatScaleInitial);
	check((int32)GetNextSmallerPositiveFloat(FloatScaleInitial) == (FloatScaleInitial - 1));

	for (uint32 y = 0; y < Height; ++y)
	{
		FSubsurfaceProfileStruct Data = SubsurfaceProfileEntries[y].Settings;

		// Fix for postload() not yet called.
		UpgradeSubsurfaceProfileParameters(Data);

		Data.Tint = Data.Tint.GetClamped();
		Data.FalloffColor = Data.FalloffColor.GetClamped(Bias);
		Data.MeanFreePathColor = Data.MeanFreePathColor.GetClamped(Bias); // In Cm
		Data.TransmissionTintColor = Data.TransmissionTintColor.GetClamped(Bias);
		Data.SurfaceAlbedo = Data.SurfaceAlbedo.GetClamped(Bias);

		// to allow blending of the Subsurface with fullres in the shader and
		// to depricate {scatter radius, falloff} in favor of albedo/MFP, need the `subsurface color` as tint.
		TextureRow[SSSS_TINT_SCALE_OFFSET] = Data.Tint;
		TextureRow[SSSS_TINT_SCALE_OFFSET].A = EncodeWorldUnitScale(Data.WorldUnitScale);

		FLinearColor DifffuseMeanFreePathInMm;

		if (Data.bEnableMeanFreePath)
		{
			DifffuseMeanFreePathInMm = 
				SubsurfaceScattering::BurleyNormalized::GetDiffuseMeanFreePathFromMeanFreePath(Data.SurfaceAlbedo, Data.MeanFreePathColor * Data.MeanFreePathDistance) * CmToMm;
		}
		else
		{
			DifffuseMeanFreePathInMm = (Data.MeanFreePathColor * Data.MeanFreePathDistance) * CmToMm;
			UE_LOGF(LogSubsurfaceProfile, Warning, "DMFP has already been upgraded to MFP. Should not reach here.");
		}

		SetupSurfaceAlbedoAndDiffuseMeanFreePath(Data.SurfaceAlbedo, DifffuseMeanFreePathInMm);
		TextureRow[BSSS_SURFACEALBEDO_OFFSET] = Data.SurfaceAlbedo;
		TextureRow[BSSS_DMFP_OFFSET] = EncodeDiffuseMeanFreePath(DifffuseMeanFreePathInMm);

		TextureRow[SSSS_BOUNDARY_COLOR_BLEED_OFFSET] = Data.BoundaryColorBleed;

		TextureRow[SSSS_BOUNDARY_COLOR_BLEED_OFFSET].A = 
			(Data.Implementation == ESubsurfaceImplementationTechniqueHint::SIH_AFIS) ? SSS_TYPE_BURLEY : SSS_TYPE_SSSS;

		float MaterialRoughnessToAverage = Data.Roughness0 * (1.0f - Data.LobeMix) + Data.Roughness1 * Data.LobeMix;

		TextureRow[SSSS_DUAL_SPECULAR_OFFSET].R = FMath::Clamp(Data.Roughness0 / SSSS_MAX_DUAL_SPECULAR_ROUGHNESS, 0.0f, 1.0f);
		TextureRow[SSSS_DUAL_SPECULAR_OFFSET].G = FMath::Clamp(Data.Roughness1 / SSSS_MAX_DUAL_SPECULAR_ROUGHNESS, 0.0f, 1.0f);
		TextureRow[SSSS_DUAL_SPECULAR_OFFSET].B = Data.LobeMix;
		TextureRow[SSSS_DUAL_SPECULAR_OFFSET].A = FMath::Clamp(MaterialRoughnessToAverage / SSSS_MAX_DUAL_SPECULAR_ROUGHNESS, 0.0f, 1.0f);

		//X:ExtinctionScale, Y:Normal Scale, Z:ScatteringDistribution, W:OneOverIOR
		TextureRow[SSSS_TRANSMISSION_OFFSET].R = EncodeExtinctionScale(Data.ExtinctionScale);
		TextureRow[SSSS_TRANSMISSION_OFFSET].G = Data.NormalScale;
		TextureRow[SSSS_TRANSMISSION_OFFSET].B = EncodeScatteringDistribution(Data.ScatteringDistribution);
		TextureRow[SSSS_TRANSMISSION_OFFSET].A = 1.0f / Data.IOR;

		if (true)
		{
			Data.ScatterRadius = FMath::Max(DifffuseMeanFreePathInMm.GetMax()*MmToCm, 0.1f);

			ComputeMirroredBSSSKernel(&TextureRow[SSSS_KERNEL0_OFFSET], SSSS_KERNEL0_SIZE, Data.SurfaceAlbedo,
				DifffuseMeanFreePathInMm, Data.ScatterRadius);
			ComputeMirroredBSSSKernel(&TextureRow[SSSS_KERNEL1_OFFSET], SSSS_KERNEL1_SIZE, Data.SurfaceAlbedo,
				DifffuseMeanFreePathInMm, Data.ScatterRadius);
			ComputeMirroredBSSSKernel(&TextureRow[SSSS_KERNEL2_OFFSET], SSSS_KERNEL2_SIZE, Data.SurfaceAlbedo,
				DifffuseMeanFreePathInMm, Data.ScatterRadius);

			Data.ScatterRadius *= (Data.WorldUnitScale * 10.0f);
		}
		else
		{
			UE_LOGF(LogSubsurfaceProfile, Warning, "Dipole model has already been upgraded to Burley. Should not reach here.");
		}

		ComputeTransmissionProfileBurley(&TextureRow[BSSS_TRANSMISSION_PROFILE_OFFSET], BSSS_TRANSMISSION_PROFILE_SIZE, 
			Data.FalloffColor, Data.ExtinctionScale, Data.SurfaceAlbedo, DifffuseMeanFreePathInMm, Data.WorldUnitScale * Data.DistanceScale, Data.TransmissionTintColor);

		// could be lower than 1 (but higher than 0) to range compress for better quality (for 8 bit)
		const float TableMaxRGB = 1.0f;
		const float TableMaxA = 3.0f;
		const FVector4f TableColorScale = FVector4f(
			1.0f / TableMaxRGB,
			1.0f / TableMaxRGB,
			1.0f / TableMaxRGB,
			1.0f / TableMaxA);

		const float CustomParameterMaxRGB = 1.0f;
		const float CustomParameterMaxA = 1.0f;
		const FVector4f CustomParameterColorScale = FVector4f(
			1.0f / CustomParameterMaxRGB,
			1.0f / CustomParameterMaxRGB,
			1.0f / CustomParameterMaxRGB,
			1.0f / CustomParameterMaxA);

		// each kernel is normalized to be 1 per channel (center + one_side_samples * 2)
		for (int32 Pos = 0; Pos < Width; ++Pos)
		{
			FVector4f C = FVector4f(TextureRow[Pos]);

			// Remap custom parameter and kernel values into 0..1
			if (Pos >= SSSS_KERNEL0_OFFSET && Pos < SSSS_KERNEL0_OFFSET + SSSS_KERNEL_TOTAL_SIZE)
			{
				C *= TableColorScale;
				// requires 16bit (could be made with 8 bit e.g. using sample0.w as 8bit scale applied to all samples (more multiplications in the shader))
				C.W *= Data.ScatterRadius / SUBSURFACE_RADIUS_SCALE;
			}
			else
			{
				C *= CustomParameterColorScale;
			}

			if (b16Bit)
			{
				// scale from 0..1 to 0..0xffff
				// scale with 0x10000 and round down to evenly distribute, avoid 0x10000

				// RGBA16 UNorm is not supported on android mobile, it falls back to RGBA16F.
				const bool bUseRGBA16F = GPixelFormats[PF_A16B16G16R16].PlatformFormat == GPixelFormats[PF_FloatRGBA].PlatformFormat;

				if (bUseRGBA16F)
				{ 
					float PlatformFloatScale = 1.0f / (FloatScaleInitial - 1.0f);

					uint16* Dest = (uint16*)(DestBuffer + DestStride * y);

					Dest[Pos * 4 + 0] = FFloat16(((uint16)(C.X * FloatScale)) * PlatformFloatScale).Encoded;
					Dest[Pos * 4 + 1] = FFloat16(((uint16)(C.Y * FloatScale)) * PlatformFloatScale).Encoded;
					Dest[Pos * 4 + 2] = FFloat16(((uint16)(C.Z * FloatScale)) * PlatformFloatScale).Encoded;
					Dest[Pos * 4 + 3] = FFloat16(((uint16)(C.W * FloatScale)) * PlatformFloatScale).Encoded;
				}
				else
				{
					uint16* Dest = (uint16*)(DestBuffer + DestStride * y);

					Dest[Pos * 4 + 0] = (uint16)(C.X * FloatScale);
					Dest[Pos * 4 + 1] = (uint16)(C.Y * FloatScale);
					Dest[Pos * 4 + 2] = (uint16)(C.Z * FloatScale);
					Dest[Pos * 4 + 3] = (uint16)(C.W * FloatScale);
				}
			}
			else
			{
				FColor* Dest = (FColor*)(DestBuffer + DestStride * y);

				Dest[Pos] = FColor(FMath::Quantize8UnsignedByte(C.X), FMath::Quantize8UnsignedByte(C.Y), FMath::Quantize8UnsignedByte(C.Z), FMath::Quantize8UnsignedByte(C.W));
			}
		}
	}

	RHICmdList.UnlockTexture2D(GSSProfiles->GetRHI(), 0, false);
}

TCHAR MiniFontCharFromIndex(uint32 Index)
{
	if (Index <= 9)
	{
		return (TCHAR)('0' + Index);
	}

	Index -= 10;

	if (Index <= 'Z' - 'A')
	{
		return (TCHAR)('A' + Index);
	}

	return (TCHAR)'?';
}

bool FSubsurfaceProfileTexture::GetEntryString(uint32 Index, FString& Out) const
{
	if (Index >= (uint32)SubsurfaceProfileEntries.Num())
	{
		return false;
	}

	const FSubsurfaceProfileStruct& ref = SubsurfaceProfileEntries[Index].Settings;


	Out = FString::Printf(TEXT(" %c. %p SurfaceAlbedo=%.1f %.1f %.1f, MeanFreePathColor=%.1f %.1f %.1f, MeanFreePathDistance=%.1f, WorldUnitScale=%.1f,\
								Tint=%.1f %.1f %.1f, BoundaryColorBleeding=%.1f %.1f %.1f, TransmissionTintColor=%.1f %.1f %.1f, %s"), 
		MiniFontCharFromIndex(Index), 
		SubsurfaceProfileEntries[Index].Profile,
		ref.SurfaceAlbedo.R, ref.SurfaceAlbedo.G, ref.SurfaceAlbedo.B,
		ref.MeanFreePathColor.R, ref.MeanFreePathColor.G, ref.MeanFreePathColor.B,
		ref.MeanFreePathDistance,
		ref.WorldUnitScale,
		ref.Tint.R, ref.Tint.G, ref.Tint.B,
		ref.BoundaryColorBleed.R, ref.BoundaryColorBleed.G, ref.BoundaryColorBleed.B, 
		ref.TransmissionTintColor.R, ref.TransmissionTintColor.G, ref.TransmissionTintColor.B,
		ref.Implementation == ESubsurfaceImplementationTechniqueHint::SIH_AFIS?TEXT("AFIS") : TEXT("Separable"));

	return true;
}

int32 FSubsurfaceProfileTexture::FindAllocationId(const USubsurfaceProfile* InProfile) const
{
	// we start at 1 because [0] is the default profile and always [0].Profile = 0 so we don't need to iterate that one
	for (int32 i = 1; i < SubsurfaceProfileEntries.Num(); ++i)
	{
		if (SubsurfaceProfileEntries[i].Profile == InProfile)
		{
			return i;
		}
	}

	return -1;
}

// for debugging
void FSubsurfaceProfileTexture::Dump()
{
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	UE_LOGF(LogSubsurfaceProfile, Log, "USubsurfaceProfileManager::Dump");
	for (int32 i = 0; i < SubsurfaceProfileEntries.Num(); ++i)
	{
		// + 1 as the Id is one higher than the array index, 0 is used for the default profile (not assigned)
		UE_LOGF(LogSubsurfaceProfile, Log, "  %d. AllocationId=%d, Pointer=%p", i, i + 1, SubsurfaceProfileEntries[i].Profile);

		{
			UE_LOGF(LogSubsurfaceProfile, Log, "     SurfaceAlbedo=%f %f %f",
				SubsurfaceProfileEntries[i].Settings.SurfaceAlbedo.R, SubsurfaceProfileEntries[i].Settings.SurfaceAlbedo.G, SubsurfaceProfileEntries[i].Settings.SurfaceAlbedo.B);
			UE_LOGF(LogSubsurfaceProfile, Log, "     MeanFreePathColor=%f %f %f",
				SubsurfaceProfileEntries[i].Settings.MeanFreePathColor.R, SubsurfaceProfileEntries[i].Settings.MeanFreePathColor.G, SubsurfaceProfileEntries[i].Settings.MeanFreePathColor.B);
			UE_LOGF(LogSubsurfaceProfile, Log, "     MeanFreePathDistance=%f",
				SubsurfaceProfileEntries[i].Settings.MeanFreePathDistance);
			UE_LOGF(LogSubsurfaceProfile, Log, "     WorldUnitScale=%f",
				SubsurfaceProfileEntries[i].Settings.WorldUnitScale);
			UE_LOGF(LogSubsurfaceProfile, Log, "     Tint=%f %f %f",
				SubsurfaceProfileEntries[i].Settings.Tint.R, SubsurfaceProfileEntries[i].Settings.Tint.G, SubsurfaceProfileEntries[i].Settings.Tint.B);
			UE_LOGF(LogSubsurfaceProfile, Log, "     Boundary Color Bleed=%f %f %f",
				SubsurfaceProfileEntries[i].Settings.BoundaryColorBleed.R, SubsurfaceProfileEntries[i].Settings.BoundaryColorBleed.G, SubsurfaceProfileEntries[i].Settings.BoundaryColorBleed.B);
			UE_LOGF(LogSubsurfaceProfile, Log, "     TransmissionTintColor=%f %f %f",
				SubsurfaceProfileEntries[i].Settings.TransmissionTintColor.R, SubsurfaceProfileEntries[i].Settings.TransmissionTintColor.G, SubsurfaceProfileEntries[i].Settings.TransmissionTintColor.B);
			UE_LOGF(LogSubsurfaceProfile, Log, "     %ls",
				SubsurfaceProfileEntries[i].Settings.Implementation == ESubsurfaceImplementationTechniqueHint::SIH_AFIS?TEXT("AFIS") : TEXT("Separable"));
		}
	}

	UE_LOGF(LogSubsurfaceProfile, Log, "");
#endif
}

FName GetSubsurfaceProfileParameterName()
{
	return SubsurfaceProfile::GetSubsurfaceProfileParameterName();
}

FName CreateSubsurfaceProfileParameterName(const USubsurfaceProfile* InProfile)
{
	return SubsurfaceProfile::CreateSubsurfaceProfileParameterName(InProfile);
}

float GetSubsurfaceProfileId(const USubsurfaceProfile* In)
{
	return SubsurfaceProfile::GetSubsurfaceProfileId(In);
}

FRHITexture* GetSubsurfaceProfileTexture()
{
	return SubsurfaceProfile::GetSubsurfaceProfileTexture();
}

FRHITexture* GetSubsurfaceProfileTextureWithFallback()
{
	return SubsurfaceProfile::GetSubsurfaceProfileTextureWithFallback();
}

FRHITexture* GetSSProfilesPreIntegratedTextureWithFallback()
{
	return SubsurfaceProfile::GetSSProfilesPreIntegratedTextureWithFallback();
}

void UpdateSubsurfaceProfileTexture(FRDGBuilder& GraphBuilder, EShaderPlatform ShaderPlatform)
{
	SubsurfaceProfile::UpdateSubsurfaceProfileTexture(GraphBuilder, ShaderPlatform);
}

namespace SubsurfaceProfile
{

FName GetSubsurfaceProfileParameterName()
{
	static FName NameSubsurfaceProfile("__SubsurfaceProfile");
	return NameSubsurfaceProfile;
}

static FName CreateSubsurfaceProfileParameterName(const FGuid& InGuid)
{
	return FName(TEXT("__SubsurfaceProfile") + InGuid.ToString());
}

FName CreateSubsurfaceProfileParameterName(const USubsurfaceProfile* InProfile)
{
	return InProfile ? CreateSubsurfaceProfileParameterName(InProfile->Guid) : FName();
}

float GetSubsurfaceProfileId(const USubsurfaceProfile* In)
{
	int32 AllocationId = 0;
	if (In)
	{
		// can be optimized (cached)
		AllocationId = GSubsurfaceProfileTextureObject.FindAllocationId(In);
	}
	else
	{
		// no profile specified means we use the default one stored at [0] which is human skin
		AllocationId = 0;
	}
	return AllocationId / 255.0f;
}

FRHITexture* GetSubsurfaceProfileTexture()
{
	return GSSProfiles ? GSSProfiles->GetRHI() : nullptr;
}

FRHITexture* GetSubsurfaceProfileTextureWithFallback()
{
	return GSSProfiles ? GSSProfiles->GetRHI() : static_cast<FRHITexture*>(GBlackTexture->TextureRHI);
}

FRHITexture* GetSSProfilesPreIntegratedTextureWithFallback()
{
	return GSSProfilesPreIntegratedTexture ? GSSProfilesPreIntegratedTexture->GetRHI() : static_cast<FRHITexture*>(GBlackArrayTexture->TextureRHI);
}

void UpdateSubsurfaceProfileTexture(FRDGBuilder& GraphBuilder, EShaderPlatform ShaderPlatform)
{
	GSubsurfaceProfileTextureObject.GetTexture(GraphBuilder.RHICmdList);
	GSubsurfaceProfileTextureObject.GetSSProfilesPreIntegratedTexture(GraphBuilder, ShaderPlatform);
}

}// namespace SubsurfaceProfile

// ------------------------------------------------------

USubsurfaceProfile::USubsurfaceProfile(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

void USubsurfaceProfile::BeginDestroy()
{
	USubsurfaceProfile* Ref = this;
	ENQUEUE_RENDER_COMMAND(RemoveSubsurfaceProfile)(
		[Ref](FRHICommandList& RHICmdList)
		{
			GSubsurfaceProfileTextureObject.RemoveProfile(Ref);
		});

	Super::BeginDestroy();
}

#if WITH_EDITOR
void USubsurfaceProfile::PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	const FSubsurfaceProfileStruct SettingsLocal = this->Settings;
	USubsurfaceProfile* Profile = this;
	GetRendererModule().InvalidatePathTracedOutput();
	ENQUEUE_RENDER_COMMAND(UpdateSubsurfaceProfile)(
		[SettingsLocal, Profile](FRHICommandListImmediate& RHICmdList)
		{
			// any changes to the setting require an update of the texture
			GSubsurfaceProfileTextureObject.UpdateProfile(SettingsLocal, Profile);
		});
}
#endif // WITH_EDITOR

void USubsurfaceProfile::PostDuplicate(EDuplicateMode::Type DuplicateMode)
{
	Super::PostDuplicate(DuplicateMode);

	// When a Subsurface Profile asset is duplicated/copied pasted (e.g. from the asset browser), we want the guid to be regenerated.
	Guid = FGuid::NewGuid();
}

void USubsurfaceProfile::PostLoad()
{
	Super::PostLoad();

	UpgradeSubsurfaceProfileParameters(this->Settings);
}

void USubsurfaceProfile::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

	Ar.UsingCustomVersion(FFortniteMainBranchObjectVersion::GUID);

	if (Ar.CustomVer(FFortniteMainBranchObjectVersion::GUID) < FFortniteMainBranchObjectVersion::SubsurfaceProfileGuid && Ar.IsLoading())
	{
		// Generate a unique GUID from the unique project asset path the first time we load an older asset.
		const FString PathName = this->GetPathName();
		Guid = FGuid::NewDeterministicGuid(PathName, 123u);
	}

	// Engine-level subsurface profile versioning
	Ar.UsingCustomVersion(FSubsurfaceProfileCustomVersion::GUID);

	if (Ar.IsLoading())
	{
		if (Ar.CustomVer(FSubsurfaceProfileCustomVersion::GUID) < FSubsurfaceProfileCustomVersion::ExtinctionTransmissionMultiplierFix)
		{
			// Previously we have 3.1f multiplied in all shaders calculating the transmission distance.
			// The ray traced shadow tries to mimic it.
			// The original purpose is to reduce the max transmission distance to 5.0/3.1 = 1.61 cm while the transmission profile is 10 times larger.
			// d * 3.1 * ExtinctionScale = d * (3.1 * ExtinctionScale)
			// Move 3.1 to extinction scale.
			Settings.ExtinctionScale *= 3.1f;

			// The transmission distance was scaled up by 10.0 in old profiles.
			Settings.DistanceScale *= 10.0f;
		}

		if (Ar.CustomVer(FSubsurfaceProfileCustomVersion::GUID) < FSubsurfaceProfileCustomVersion::FactorExtraSurfaceAlbedoToTransmissionTintFix)
		{
			// Transmission has SurfaceAlbedo * StoredBaseColor previously.
			Settings.TransmissionTintColor *= Settings.SurfaceAlbedo;
		}
	}
}

