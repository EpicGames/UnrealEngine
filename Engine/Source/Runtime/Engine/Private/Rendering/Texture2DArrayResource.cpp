// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	Texture2DArrayResource.cpp: Implementation of FTexture2DArrayResource used  by streamable UTexture2DArray.
=============================================================================*/

#include "Rendering/Texture2DArrayResource.h"
#include "EngineLogs.h"
#include "Engine/Texture2DArray.h"
#include "RenderUtils.h"

static const FName NAME_FTexture2DArrayResource("FTexture2DArrayResource");

//*****************************************************************************
//************************* FTexture2DArrayResource ***************************
//*****************************************************************************

FTexture2DArrayResource::FTexture2DArrayResource(UTexture2DArray* InOwner, const FStreamableRenderResourceState& InState) 
: FStreamableTextureResource(InOwner, InOwner->GetPlatformData(), InState, false)
{
	AddressU = InOwner->AddressX == TA_Wrap ? AM_Wrap : (InOwner->AddressX == TA_Clamp ? AM_Clamp : AM_Mirror);
	AddressV = InOwner->AddressY == TA_Wrap ? AM_Wrap : (InOwner->AddressY == TA_Clamp ? AM_Clamp : AM_Mirror);
	AddressW = InOwner->AddressZ == TA_Wrap ? AM_Wrap : (InOwner->AddressZ == TA_Clamp ? AM_Clamp : AM_Mirror);
	
	//
	// All resource requests assume LOD bias is baked in - however GetMipData doesn't, and we don't want to grab
	// that data if we don't have to. LODCountToAssetFirstLODIdx() is the index with LODBias, and RequestedFirstLODIdx()
	// isn't. (LODCountToAssetFirstLODIdx = RequestedFirstLODIdx + AssetLODBias).
	//
	if (InOwner->GetMipData(State.LODCountToAssetFirstLODIdx(State.NumRequestedLODs), AllMipsData) == false)
	{
		// This is fatal as we will crash trying to upload the data below, this way we crash at the cause.
		UE_LOGF(LogTexture, Fatal, "Corrupt texture [%ls]! Unable to load mips (bulk data missing)", *TextureName.ToString());
		return;
	}
	
	// AllMipsData has NumRequestedLODs - (GetNumMipsInTail() - 1)
	{
		// If zero, then it's unpacked which means there's 1 mip in the tail.
		const int32 MipsInTail = PlatformData->GetNumMipsInTail() ? PlatformData->GetNumMipsInTail() : 1;
		const int32 MipCountLostDueToPacking = MipsInTail - 1;
		check(AllMipsData.Num() == State.NumRequestedLODs - MipCountLostDueToPacking);
	}
}

FTexture2DArrayResource::FTexture2DArrayResource(UTexture2DArray* InOwner, const FTexture2DArrayResource* InProxiedResource)
	: FStreamableTextureResource(InOwner, InProxiedResource->PlatformData, FStreamableRenderResourceState(), false)
	, ProxiedResource(InProxiedResource)
{
}

void FTexture2DArrayResource::CreateTexture(FRHICommandListBase& RHICmdList)
{
	TArrayView<const FTexture2DMipMap*> MipsViewPostLODBias = GetPlatformMipsView();
	const FTexture2DMipMap& FirstMip = *MipsViewPostLODBias[State.RequestedFirstLODIdx()];

	const FString Name = TextureName.ToString();

	const FRHITextureCreateDesc Desc =
		FRHITextureCreateDesc::Create2DArray(*Name, FirstMip.SizeX, FirstMip.SizeY, FirstMip.SizeZ, PixelFormat)
		.SetNumMips(State.NumRequestedLODs)
		.SetFlags(CreationFlags)
		.SetExtData(PlatformData->GetExtData())
		.SetClassName(NAME_FTexture2DArrayResource)
		.SetOwnerName(GetOwnerName());

	TextureRHI = RHICmdList.CreateTexture(Desc);

	// Copy the mip data in to the texture.
	for (int32 MipIndex = 0; MipIndex < AllMipsData.Num(); MipIndex++)
	{		
		for (uint32 ArrayIndex = 0; ArrayIndex < SizeZ; ++ArrayIndex)
		{
			uint32 DestStride = 0;
			void* DestData = RHICmdList.GetAsImmediate().LockTexture2DArray(TextureRHI, ArrayIndex, MipIndex, RLM_WriteOnly, DestStride, false);
			if (DestData)
			{
				GetData(FirstMip.SizeX, FirstMip.SizeY, ArrayIndex, MipIndex, DestData, DestStride);
			}
			else
			{
				UE_LOGF(LogTexture, Warning, "Failed to lock texture 2d array mip/slice %d / %d (%ls)", MipIndex, ArrayIndex, *TextureName.ToString());
			}
			RHICmdList.GetAsImmediate().UnlockTexture2DArray(TextureRHI, ArrayIndex, MipIndex, false);
		}
	}

	AllMipsData.Empty();
}

void FTexture2DArrayResource::CreatePartiallyResidentTexture(FRHICommandListBase& RHICmdList)
{
	unimplemented();
	TextureRHI.SafeRelease();
}

uint64 FTexture2DArrayResource::GetPlatformMipsSize(uint32 NumMips) const
{
	if (PlatformData && NumMips > 0)
	{
		const FIntPoint MipExtents = CalcMipMapExtent(SizeX, SizeY, PixelFormat, State.LODCountToFirstLODIdx(NumMips));

		const FRHITextureDesc Desc =
			FRHITextureCreateDesc::Create2DArray(TEXT("Temp"), MipExtents, SizeZ, PixelFormat)
			.SetNumMips(NumMips)
			.SetFlags(CreationFlags)
			.SetExtData(PlatformData->GetExtData());

		return RHICalcTexturePlatformSize(Desc).Size;
	}

	return 0;
}

void FTexture2DArrayResource::InitRHI(FRHICommandListBase& RHICmdList)
{
	if (ProxiedResource)
	{
		TextureRHI = ProxiedResource->TextureRHI;
		RHICmdList.UpdateTextureReference(TextureReferenceRHI, TextureRHI);
		SamplerStateRHI = ProxiedResource->SamplerStateRHI;
		DeferredPassSamplerStateRHI = ProxiedResource->DeferredPassSamplerStateRHI;
	}
	else
	{
		FStreamableTextureResource::InitRHI(RHICmdList);
	}
}

void FTexture2DArrayResource::GetData(int32 BaseRHIMipSizeX, int32 BaseRHIMipSizeY, uint32 ArrayIndex, uint32 MipIndex, void* Dest, uint32 DestPitch) const
{
	const uint32 ArrayCount = SizeZ;
	const FMemoryView MipView = AllMipsData[MipIndex].GetView();
	const uint64 ArraySliceDataSize = MipView.GetSize() / ArrayCount;
	const FMemoryView& SliceMipDataView = MipView.Mid(ArrayIndex * ArraySliceDataSize, ArraySliceDataSize);

	
	// This check works for normal mips but not packed mips.
	//check(ArraySliceDataSize == CalcTextureMipMapSize(BaseRHIMipSizeX, BaseRHIMipSizeX, PixelFormat, MipIndex));

	// for platforms that returned 0 pitch from Lock, we need to just use the bulk data directly, never do 
	// runtime block size checking, conversion, or the like
	if (DestPitch == 0)
	{
		FMemory::Memcpy(Dest, SliceMipDataView.GetData(), SliceMipDataView.GetSize());
	}
	else
	{
		const int32 BlockSizeX = GPixelFormats[PixelFormat].BlockSizeX;
		const int32 BlockSizeY = GPixelFormats[PixelFormat].BlockSizeY;
		const int32 BlockBytes = GPixelFormats[PixelFormat].BlockBytes;

		const uint32 MipSizeX = FMath::Max<int32>(BaseRHIMipSizeX >> MipIndex, 1);
		const uint32 MipSizeY = FMath::Max<int32>(BaseRHIMipSizeY >> MipIndex, 1);

		uint32 NumColumns = FMath::DivideAndRoundUp<int32>(MipSizeX, BlockSizeX);
		uint32 NumRows = FMath::DivideAndRoundUp<int32>(MipSizeY, BlockSizeY);

		if (PixelFormat == PF_PVRTC2 || PixelFormat == PF_PVRTC4)
		{
			// PVRTC has minimum 2 blocks width and height
			NumColumns = FMath::Max<uint32>(NumColumns, 2);
			NumRows = FMath::Max<uint32>(NumRows, 2);
		}
		const uint32 SrcStride = NumColumns * BlockBytes;

		CopyTextureData2D(SliceMipDataView.GetData(), Dest, MipSizeY, PixelFormat, SrcStride, DestPitch);
	}
}
