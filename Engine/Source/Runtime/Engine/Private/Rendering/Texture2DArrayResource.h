// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

/*=============================================================================
	Texture2DArrayResource.cpp: Implementation of FTexture2DArrayResource used  by streamable UTexture2DArray.
=============================================================================*/

#include "CoreMinimal.h"
#include "Rendering/StreamableTextureResource.h"
#include "Containers/ResourceArray.h"
#include "Memory/SharedBuffer.h"

class UTexture2DArray;

/** Represents a 2D Texture Array to the renderer. */
class FTexture2DArrayResource : public FStreamableTextureResource
{
public:

	FTexture2DArrayResource(UTexture2DArray* InOwner, const FStreamableRenderResourceState& InState);

	/**
	 * Make this Texture2DArrayResource Proxy another one.
	 *
	 * @param InOwner             UTexture2DArray which this FTexture2DArrayResource represents.
	 * @param InProxiedResource   The resource to proxy.
	 */
	FTexture2DArrayResource(UTexture2DArray* InOwner, const FTexture2DArrayResource* InProxiedResource);

	// Dynamic cast methods.
	virtual FTexture2DArrayResource* GetTexture2DArrayResource() { return this; }
	// Dynamic cast methods (const).
	virtual const FTexture2DArrayResource* GetTexture2DArrayResource() const { return this; }

	virtual uint64 GetPlatformMipsSize(uint32 NumMips) const override;

	virtual void InitRHI(FRHICommandListBase& RHICmdList) override;
	virtual bool IsProxy() const override { return ProxiedResource != nullptr; }

protected:

	void CreateTexture(FRHICommandListBase& RHICmdList) final override;
	void CreatePartiallyResidentTexture(FRHICommandListBase& RHICmdList) final override;

	void GetData(int32 BaseRHIMipSizeX, int32 BaseRHIMipSizeY, uint32 ArrayIndex, uint32 MipIndex, void* Dest, uint32 DestPitch) const;

	/** Another resource being proxied by this one. */
	const FTexture2DArrayResource* const ProxiedResource = nullptr;

	// Each mip has all array slices. This will be [State.NumRequestedLODs] long, less any packed mips.
	TArray<FUniqueBuffer, TInlineAllocator<MAX_TEXTURE_MIP_COUNT>> AllMipsData;
};
