// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	Texture2DResource.cpp: Implementation of FTexture2DResource used  by streamable UTexture2D.
=============================================================================*/

#include "Rendering/Texture2DResource.h"
#include "EngineLogs.h"
#include "Engine/Texture2D.h"
#include "RenderUtils.h"
#include "RenderingThread.h"
#include "Containers/ResourceArray.h"

const static FName NAME_FTexture2DResource("FTexture2DResource");

// TODO Only adding this setting to allow backwards compatibility to be forced.  The default
// behavior is to NOT do this.  This variable should be removed in the future.  #ADDED 4.13
static TAutoConsoleVariable<int32> CVarForceHighestMipOnUITexturesEnabled(
	TEXT("r.ForceHighestMipOnUITextures"),
	0,
	TEXT("If set to 1, texutres in the UI Group will have their highest mip level forced."),
	ECVF_RenderThreadSafe);

/**
 * Minimal initialization constructor.
 *
 * @param InOwner			UTexture2D which this FTexture2DResource represents.
 * @param InState			
 */
FTexture2DResource::FTexture2DResource(UTexture2D* InOwner, const FStreamableRenderResourceState& InState)
	: FStreamableTextureResource(InOwner, InOwner->GetPlatformData(), InState, true)
	, ResourceMem(InOwner->ResourceMem)
{
	// Retrieve initial mip data.
	MipData.AddZeroed(State.MaxNumLODs);
	MipDataSize.AddZeroed(State.MaxNumLODs);

	int32 AssetFirstMipToLoad = State.LODCountToAssetFirstLODIdx(State.NumRequestedLODs);	// takes lod bias into account
	int32 LocalFirstMipToLoad = State.LODCountToFirstLODIdx(State.NumRequestedLODs);		// ignores bias
	InOwner->GetInitialMipData(
		AssetFirstMipToLoad,
		TArrayView<void*>(MipData).RightChop(LocalFirstMipToLoad),
		TArrayView<int64>(MipDataSize).RightChop(LocalFirstMipToLoad));

	CacheSamplerStateInitializer(InOwner);
}

FTexture2DResource::FTexture2DResource(UTexture2D* InOwner, const FTexture2DResource* InProxiedResource)
	: FStreamableTextureResource(InOwner, InProxiedResource->PlatformData, FStreamableRenderResourceState(), true)
	, ResourceMem(InOwner->ResourceMem)
	, ProxiedResource(InProxiedResource)
{
	CacheSamplerStateInitializer(InOwner);
}

/**
 * Destructor, freeing MipData in the case of resource being destroyed without ever 
 * having been initialized by the rendering thread via InitRHI.
 */
FTexture2DResource::~FTexture2DResource()
{
	// free resource memory that was preallocated
	// The deletion needs to happen in the rendering thread.
	FTexture2DResourceMem* InResourceMem = ResourceMem;
	ENQUEUE_RENDER_COMMAND(DeleteResourceMem)(
		[InResourceMem](FRHICommandList& RHICmdList)
		{
			delete InResourceMem;
		});

	// Make sure we're not leaking memory if InitRHI has never been called.
	for( int32 MipIndex=0; MipIndex<MipData.Num(); MipIndex++ )
	{
		// free any mip data that was copied 
		if( MipData[MipIndex] )
		{
			FMemory::Free( MipData[MipIndex] );
		}
		MipData[MipIndex] = NULL;
		MipDataSize[MipIndex] = 0;
	}
}

void FTexture2DResource::CacheSamplerStateInitializer(const UTexture2D* InOwner)
{
	float DefaultMipBias = 0;
	if (PlatformData && LODGroup == TEXTUREGROUP_UI && CVarForceHighestMipOnUITexturesEnabled.GetValueOnAnyThread() > 0)
	{
		DefaultMipBias = -PlatformData->Mips.Num();
	}

	// Set FStreamableTextureResource sampler state settings as it is UTexture2D specific.
	AddressU = InOwner->AddressX == TA_Wrap ? AM_Wrap : (InOwner->AddressX == TA_Clamp ? AM_Clamp : AM_Mirror);
	AddressV = InOwner->AddressY == TA_Wrap ? AM_Wrap : (InOwner->AddressY == TA_Clamp ? AM_Clamp : AM_Mirror);
	MipBias = UTexture2D::GetGlobalMipMapLODBias() + DefaultMipBias;
}

void FTexture2DResource::InitRHI(FRHICommandListBase& RHICmdList)
{
	if (ProxiedResource)
	{
		TextureRHI = ProxiedResource->TextureRHI;
		RHICmdList.UpdateTextureReference(TextureReferenceRHI, TextureRHI);
		SamplerStateRHI = ProxiedResource->SamplerStateRHI;
		DeferredPassSamplerStateRHI = ProxiedResource->DeferredPassSamplerStateRHI;
	}
	else
	{
		FStreamableTextureResource::InitRHI(RHICmdList);
	}
}

void FTexture2DResource::CreateTexture(FRHICommandListBase& RHICmdList)
{
	const int32 RequestedMipIdx = State.RequestedFirstLODIdx();
	const FTexture2DMipMap* RequestedMip = GetPlatformMip(RequestedMipIdx);

	const FString Name = TextureName.ToString();

	// create texture with ResourceMem data when available
	FRHITextureCreateDesc Desc =
		FRHITextureCreateDesc::Create2D(*Name, RequestedMip->SizeX, RequestedMip->SizeY, PixelFormat)
		.SetNumMips(State.NumRequestedLODs)
		.SetFlags(CreationFlags)
		.SetExtData(PlatformData->GetExtData())
		.SetClassName(NAME_FTexture2DResource)
		.SetOwnerName(GetOwnerName());

	if (ResourceMem)
	{
		Desc.SetInitActionBulkData(ResourceMem);
	}

	TextureRHI = RHICmdList.CreateTexture(Desc);

	// if( ResourceMem && !State.bReadyForStreaming) // To investigate!
	if (ResourceMem)
	{
		// when using resource memory the RHI texture has already been initialized with data and won't need to have mips copied
		check(State.NumRequestedLODs == ResourceMem->GetNumMips());
		check(RequestedMip->SizeX == ResourceMem->GetSizeX() && RequestedMip->SizeY == ResourceMem->GetSizeY());
		for( int32 MipIndex=0; MipIndex < MipData.Num(); MipIndex++ )
		{
			MipData[MipIndex] = nullptr;
			MipDataSize[MipIndex] = 0;
		}
	}
	else
	{
		// Read the resident mip-levels into the RHI texture.
		for (int32 RHIMipIdx = 0; RHIMipIdx < State.NumRequestedLODs; ++RHIMipIdx)
		{
			const int32 ResourceMipIdx = RHIMipIdx + RequestedMipIdx;
			if (MipData[ResourceMipIdx])
			{
				uint32 DestPitch = -1;
				uint64 Size = ~0ULL;
				void* TheMipData = RHICmdList.GetAsImmediate().LockTexture2D(TextureRHI, RHIMipIdx, RLM_WriteOnly, DestPitch, false /* bLockWithinMiptail */, true /* bFlushRHIThread */, &Size);
				//check(Size != ~0ULL);

				GetData( ResourceMipIdx, TheMipData, DestPitch, Size );
				RHICmdList.GetAsImmediate().UnlockTexture2D(TextureRHI, RHIMipIdx, false );
			}
		}
	}
}

void FTexture2DResource::CreatePartiallyResidentTexture(FRHICommandListBase& RHICmdList)
{
	const FString Name = TextureName.ToString();
	const int32 CurrentFirstMip = State.RequestedFirstLODIdx();

	check(bUsePartiallyResidentMips);

	const FRHITextureCreateDesc Desc =
		FRHITextureCreateDesc::Create2D(*Name, SizeX, SizeY, PixelFormat)
		.SetNumMips(State.MaxNumLODs)
		.SetFlags(CreationFlags | ETextureCreateFlags::Virtual)
		.SetExtData(PlatformData->GetExtData())
		.SetClassName(NAME_FTexture2DResource)
		.SetOwnerName(GetOwnerName());

	TextureRHI = RHICmdList.CreateTexture(Desc);

	RHICmdList.GetAsImmediate().VirtualTextureSetFirstMipInMemory(TextureRHI, CurrentFirstMip);
	RHICmdList.GetAsImmediate().VirtualTextureSetFirstMipVisible(TextureRHI, CurrentFirstMip);

	check( ResourceMem == nullptr );

	// Read the resident mip-levels into the RHI texture.
	for( int32 MipIndex=CurrentFirstMip; MipIndex < State.MaxNumLODs; MipIndex++ )
	{
		if ( MipData[MipIndex] != NULL )
		{
			uint32 DestPitch = -1;
			uint64 Size = ~0ULL;
			void* TheMipData = RHICmdList.GetAsImmediate().LockTexture2D(TextureRHI, MipIndex, RLM_WriteOnly, DestPitch, false /* bLockWithinMiptail */, true /* bFlushRHIThread */, &Size);
			//check(Size != ~0ULL);

			GetData( MipIndex, TheMipData, DestPitch, Size);
			RHICmdList.GetAsImmediate().UnlockTexture2D(TextureRHI, MipIndex, false);
		}
	}
}

uint64 FTexture2DResource::GetPlatformMipsSize(uint32 NumMips) const
{
	if (PlatformData && NumMips > 0)
	{
		static TConsoleVariableData<int32>* CVarReducedMode = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.PRT.ReducedMemory"));
		check(CVarReducedMode);

		FRHICalcTextureSizeResult Result{};

		uint32 TextureAlign = 0;
		// Must be consistent with the logic in FTexture2DResource::InitRHI
		if (bUsePartiallyResidentMips && (!CVarReducedMode->GetValueOnRenderThread() || NumMips > State.NumNonStreamingLODs))
		{
			const FRHITextureDesc Desc =
				FRHITextureCreateDesc::Create2D(TEXT("Temp"), SizeX, SizeY, PixelFormat)
				.SetNumMips(State.MaxNumLODs)
				.SetFlags(CreationFlags | TexCreate_Virtual);

			Result = RHICalcTexturePlatformSize(Desc, State.LODCountToFirstLODIdx(NumMips));
		}
		else
		{
			const FIntPoint MipExtents = CalcMipMapExtent(SizeX, SizeY, PixelFormat, State.RequestedFirstLODIdx());

			const FRHITextureDesc Desc =
				FRHITextureCreateDesc::Create2D(TEXT("Temp"), MipExtents, PixelFormat)
				.SetNumMips(NumMips)
				.SetFlags(CreationFlags)
				.SetExtData(PlatformData->GetExtData());

			Result = RHICalcTexturePlatformSize(Desc);
		}

		return Result.Size;
	}

	return 0;
}

uint32 FTexture2DResource::CalculateTightPackedMipSize(int32 MipSizeX, int32 MipSizeY, EPixelFormat PixelFormat, uint32& OutPitch)
{
	const FPixelFormatInfo& Format = GPixelFormats[PixelFormat];

	uint32 NumColumns = FMath::DivideAndRoundUp(MipSizeX, Format.BlockSizeX); // Num-of columns in the source data (in blocks)
	uint32 NumRows    = FMath::DivideAndRoundUp(MipSizeY, Format.BlockSizeY); // Num-of rows in the source data (in blocks)

	if (PixelFormat == PF_PVRTC2 || PixelFormat == PF_PVRTC4)
	{
		// PVRTC has minimum 2 blocks width and height
		NumColumns = FMath::Max<uint32>(NumColumns, 2);
		NumRows = FMath::Max<uint32>(NumRows, 2);
	}

	const uint32 SrcPitch   = NumColumns * Format.BlockBytes; // Num-of bytes per row in the source data
	const uint32 EffectiveSize = Format.BlockBytes * NumColumns * NumRows;

	OutPitch = SrcPitch;
	return EffectiveSize;
}

void FTexture2DResource::WarnRequiresTightPackedMip(int32 SizeX,int32 SizeY,EPixelFormat PixelFormat,
	uint32 Pitch)
{
	uint32 ExpectedPitch;
	uint32 ExpectedSize = CalculateTightPackedMipSize(SizeX,SizeY,PixelFormat,ExpectedPitch);

	if ( Pitch != 0 && Pitch != ExpectedPitch )
	{
		UE_LOGF(LogTexture,Warning,"Requires tight packed pitch, expected: %d (%dx%d = %d total), saw: %d",
			ExpectedPitch,SizeX,SizeY,ExpectedSize,Pitch);
	}
}

/**
 * Writes the data for a single mip-level into a destination buffer.
 *
 * @param MipIndex		Index of the mip-level to read.
 * @param Dest			Address of the destination buffer to receive the mip-level's data.
 * @param DestPitch		Number of bytes per row
 * @param DestSize		Number of bytes locked by RHILockTexture2D.
 */
void FTexture2DResource::GetData( uint32 MipIndex, void* Dest, uint32 DestPitch, uint64 DestSize )
{
	const FTexture2DMipMap& MipMap = *GetPlatformMip(MipIndex);
	check( MipData[MipIndex] != nullptr );
	
	uint32 SrcPitch;
	uint32 EffectiveSize = CalculateTightPackedMipSize(MipMap.SizeX,MipMap.SizeY,PixelFormat,SrcPitch);

	int64 DataSize = MipDataSize[MipIndex];

	UE_LOGF(LogTextureUpload,Verbose,"Size: %dx%d , EffectiveSize=%d BulkDataSize=%lld , SrcPitch=%d DestPitch=%d Format=%d, DestSize=%lld",
		MipMap.SizeX,MipMap.SizeY,
		EffectiveSize,DataSize,
		SrcPitch,DestPitch, PixelFormat, DestSize);

	if ((uint64)DataSize > DestSize)
	{
		UE_LOGF(LogTextureUpload, Warning, "DestSize is reported smaller than BulkDataSize in upload! Either RHI is reporting wrong sizes or we are stomping memory! Size: %dx%d, EffectiveSize=%d, BulkDataSize=%lld, SrcPitch=%d, DestPitch=%d, Format=%d, DestSize=%lld",
			MipMap.SizeX,MipMap.SizeY,
			EffectiveSize,DataSize,
			SrcPitch,DestPitch, PixelFormat, DestSize);
		DataSize  = DestSize; // don't overwrite memory
	}

	// for platforms that returned 0 pitch from Lock, we need to just use the bulk data directly, never do 
	// runtime block size checking, conversion, or the like
	if (DestPitch == 0)
	{
		FMemory::Memcpy(Dest, MipData[MipIndex], DataSize);
	}
	else
	{
		#if WITH_EDITORONLY_DATA
		// in Editor, Mip doesn't come from BulkData, it may be null
		// MipData[] was set from Editor data
		// would be nice to check MipData[MipIndex] size ! but it's not stored
		if (DataSize == 0)
		{
			DataSize = EffectiveSize;
		}
		#endif

	#if !WITH_EDITORONLY_DATA
		// only checking when !WITH_EDITORONLY_DATA ? because in Editor BulkDataSize == 0 , so not possible to check
		checkf((int64)EffectiveSize == DataSize, 
			TEXT("Texture '%s', mip %d, has a DataSize [%lld] that doesn't match calculated size [%d]. Texture size %dx%d, format %d"),
			*TextureName.ToString(), MipIndex, DataSize, EffectiveSize, GetSizeX(), GetSizeY(), (int32)PixelFormat);
	#endif

		// for platforms that returned 0 pitch from Lock, we need to just use the bulk data directly, never do 
		// runtime block size checking, conversion, or the like
		if (DestPitch == 0 || DestPitch == SrcPitch )
		{
			// checking Dest size before we memcpy would be nice!
			FMemory::Memcpy(Dest, MipData[MipIndex], DataSize);
		}
		else
		{
			// Copy the texture data.
			CopyTextureData2D(MipData[MipIndex],Dest,MipMap.SizeY,PixelFormat,SrcPitch,DestPitch);
		}
	}
	
	// Free data retrieved via GetCopy inside constructor.
	FMemory::Free(MipData[MipIndex]);
	MipData[MipIndex] = NULL;
	MipDataSize[MipIndex] = 0;
}
