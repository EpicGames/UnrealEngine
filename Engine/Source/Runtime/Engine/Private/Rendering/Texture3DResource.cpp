// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	Texture3DResource.cpp: Implementation of FTexture3DResource used  by streamable UVolumeTexture.
=============================================================================*/

#include "Rendering/Texture3DResource.h"
#include "Engine/VolumeTexture.h"
#include "EngineLogs.h"
#include "RenderUtils.h"

static const FName NAME_FTexture3DResource("FTexture3DResource");

//*****************************************************************************
//************************* FVolumeTextureBulkData ****************************
//*****************************************************************************

void FVolumeTextureBulkData::Discard()
{
	for (int32 MipIndex = 0; MipIndex < MAX_TEXTURE_MIP_COUNT; ++MipIndex)
	{
		if (MipData[MipIndex])
		{
			FMemory::Free(MipData[MipIndex]);
			MipData[MipIndex] = nullptr;
		}
		MipSize[MipIndex] = 0;
	}
}

void FVolumeTextureBulkData::MergeMips(int32 NumMips)
{
	check(NumMips < MAX_TEXTURE_MIP_COUNT);

	int64 MergedSize = 0;
	for (int32 MipIndex = FirstMipIdx; MipIndex < NumMips; ++MipIndex)
	{
		MergedSize += MipSize[MipIndex];
	}

	// Don't do anything if there is nothing to merge
	if (MergedSize > MipSize[FirstMipIdx])
	{
		uint8* MergedAlloc = (uint8*)FMemory::Malloc(MergedSize, MALLOC_ALIGNMENT);
		uint8* CurrPos = MergedAlloc;
		for (int32 MipIndex = FirstMipIdx; MipIndex < NumMips; ++MipIndex)
		{
			if (MipData[MipIndex])
			{
				FMemory::Memcpy(CurrPos, MipData[MipIndex], MipSize[MipIndex]);
			}
			CurrPos += MipSize[MipIndex];
		}

		Discard();

		MipData[FirstMipIdx] = MergedAlloc;
		MipSize[FirstMipIdx] = MergedSize;
	}
}

//*****************************************************************************
//*************************** FTexture3DResource ******************************
//*****************************************************************************

FTexture3DResource::FTexture3DResource(UVolumeTexture* InOwner, const FStreamableRenderResourceState& InState)
: FStreamableTextureResource(InOwner, InOwner->GetPlatformData(), InState, false)
, InitialData(InState.RequestedFirstLODIdx())
{
	if (!GRHIGlobals.SupportsTexture3DAnisotropicFiltering)
	{
		MaxAniso = 1;
	}

	const int32 FirstLODIdx = InState.RequestedFirstLODIdx();

	// changed to use TryLoadMipsWithSizes
	if (PlatformData && const_cast<FTexturePlatformData*>(PlatformData)->TryLoadMipsWithSizes(FirstLODIdx + InState.AssetLODBias, 
		InitialData.GetMipData() + FirstLODIdx, InitialData.GetMipSize() + FirstLODIdx, InOwner->GetPathName()))
	{
		// Compute the size of each mips so that they can be merged into a single allocation.
		// -> this should be unnecessary now that TryLoadMipsWithSizes reports the sizes
		if (GRHIGlobals.UseTexture3DBulkData)
		{
			for (int32 MipIndex = FirstLODIdx; MipIndex < InState.MaxNumLODs; ++MipIndex)
			{
				const FTexture2DMipMap& MipMap = PlatformData->Mips[MipIndex];
				
				uint32 MipExtentX = 0;
				uint32 MipExtentY = 0;
				uint32 MipExtentZ = 0;
				CalcMipMapExtent3D(SizeX, SizeY, SizeZ, PixelFormat, State.RequestedFirstLODIdx(), MipExtentX, MipExtentY, MipExtentZ);

				const FRHITextureDesc Desc =
					FRHITextureCreateDesc::Create3D(TEXT("Temp"), MipExtentX, MipExtentY, MipExtentZ, PixelFormat)
					.SetNumMips(State.NumRequestedLODs)
					.SetFlags(CreationFlags)
					.SetExtData(PlatformData->GetExtData());

				const uint64 PlatformMipSize = RHICalcTexturePlatformSize(Desc).Size;

				UE_LOGF(LogTexture,Verbose,"FTexture3DResource::FTexture3DResource %d : %dx%dx%d : MipBytes=%d",
					MipIndex,MipExtentX,MipExtentY,MipExtentZ,
					(int)PlatformMipSize);

				// The bulk data can be bigger because of memory alignment constraints on each slice and mips.
				int64 BulkDataSize = MipMap.BulkData.GetBulkDataSize();
				int64 CalcMipSize = CalcTextureMipMapSize3D(SizeX, SizeY, SizeZ, (EPixelFormat)PixelFormat, MipIndex);
				check( BulkDataSize >= CalcMipSize );
				//InitialData.GetMipSize()[MipIndex] = BulkDataSize;
				check( InitialData.GetMipSize()[MipIndex] == BulkDataSize );
			}
		}

	}
}

FTexture3DResource::FTexture3DResource(UVolumeTexture* InOwner, const FTexture3DResource* InProxiedResource)
	: FStreamableTextureResource(InOwner, InProxiedResource->PlatformData, FStreamableRenderResourceState(), false)
	, ProxiedResource(InProxiedResource)
	, InitialData(0)
{
	if (!GRHIGlobals.SupportsTexture3DAnisotropicFiltering)
	{
		MaxAniso = 1;
	}
}

void FTexture3DResource::CreateTexture(FRHICommandListBase& RHICmdList)
{
	TArrayView<const FTexture2DMipMap*> MipsView = GetPlatformMipsView();
	const int32 FirstMipIdx = InitialData.GetFirstMipIdx(); // == State.RequestedFirstLODIdx()

	// Create the RHI texture.
	{
		const FTexture2DMipMap& FirstMip = *MipsView[FirstMipIdx];
		const FString Name = TextureName.ToString();

		FRHITextureCreateDesc Desc =
			FRHITextureCreateDesc::Create3D(*Name, FirstMip.SizeX, FirstMip.SizeY, FirstMip.SizeZ, PixelFormat)
			.SetNumMips(State.NumRequestedLODs)
			.SetFlags(CreationFlags)
			.SetExtData(PlatformData->GetExtData())
			.SetClassName(NAME_FTexture3DResource)
			.SetOwnerName(GetOwnerName());

		if (GRHIGlobals.UseTexture3DBulkData)
		{
			InitialData.MergeMips(State.MaxNumLODs);
			Desc.SetInitActionBulkData(&InitialData);
		}

		Desc.DetermineInititialState();

		TextureRHI = RHICmdList.CreateTexture(Desc);
	}

	if (!GRHIGlobals.UseTexture3DBulkData) 
	{
		const int32 BlockSizeX = GPixelFormats[PixelFormat].BlockSizeX;
		const int32 BlockSizeY = GPixelFormats[PixelFormat].BlockSizeY;
		const int32 BlockBytes = GPixelFormats[PixelFormat].BlockBytes;
		ensure(GPixelFormats[PixelFormat].BlockSizeZ == 1);

		for (int32 RHIMipIdx = 0; RHIMipIdx < State.NumRequestedLODs; ++RHIMipIdx)
		{
			const int32 ResourceMipIdx = RHIMipIdx + FirstMipIdx;
			const FTexture2DMipMap& Mip = *MipsView[ResourceMipIdx];
			const uint8* MipData = (const uint8*)InitialData.GetMipData()[ResourceMipIdx];
			if (MipData)
			{
				FUpdateTextureRegion3D UpdateRegion(0, 0, 0, 0, 0, 0, Mip.SizeX, Mip.SizeY, Mip.SizeZ);

				// RHICmdList.UpdateTexture3D crashes on some platforms at engine initialization time.
				// The default volume texture end up being loaded at that point, which is a problem.
				// We check if this is really the rendering thread to find out if the engine is initializing.
				const uint32 NumBlockX = (uint32)FMath::DivideAndRoundUp<int32>(Mip.SizeX, BlockSizeX);
				const uint32 NumBlockY = (uint32)FMath::DivideAndRoundUp<int32>(Mip.SizeY, BlockSizeY);

				{
				int64 MipBytes = InitialData.GetMipSize()[ResourceMipIdx];
				int64 UploadSize = NumBlockX * NumBlockY * BlockBytes * Mip.SizeZ;
				check( MipBytes >= UploadSize );

				UE_LOGF(LogTexture,Verbose,"FTexture3DResource::CreateTexture:UpdateTexture3D %d : %dx%dx%d : RowStride=%d, SliceStride=%d, MipBytes=%d",
					ResourceMipIdx,Mip.SizeX,Mip.SizeY,Mip.SizeZ,
					NumBlockX * BlockBytes, NumBlockX * NumBlockY * BlockBytes,
					(int)MipBytes);
				}

				RHICmdList.UpdateTexture3D(TextureRHI, RHIMipIdx, UpdateRegion, NumBlockX * BlockBytes, NumBlockX * NumBlockY * BlockBytes, MipData);
			}
		}
		InitialData.Discard();
	}
}

void FTexture3DResource::CreatePartiallyResidentTexture(FRHICommandListBase& RHICmdList)
{
	unimplemented();
	TextureRHI.SafeRelease();
}

uint64 FTexture3DResource::GetPlatformMipsSize(uint32 NumMips) const
{
	if (PlatformData && NumMips > 0)
	{
		uint32 MipExtentX = 0;
		uint32 MipExtentY = 0;
		uint32 MipExtentZ = 0;
		CalcMipMapExtent3D(SizeX, SizeY, SizeZ, PixelFormat, State.LODCountToFirstLODIdx(NumMips), MipExtentX, MipExtentY, MipExtentZ);

		const FRHITextureDesc Desc =
			FRHITextureCreateDesc::Create3D(TEXT("Temp"), MipExtentX, MipExtentY, MipExtentZ, PixelFormat)
			.SetNumMips(NumMips)
			.SetFlags(CreationFlags)
			.SetExtData(PlatformData->GetExtData());

		return RHICalcTexturePlatformSize(Desc).Size;
	}

	return 0;
}

void FTexture3DResource::InitRHI(FRHICommandListBase& RHICmdList)
{
	if (ProxiedResource)
	{
		TextureRHI = ProxiedResource->TextureRHI;
		RHICmdList.UpdateTextureReference(TextureReferenceRHI, TextureRHI);
		SamplerStateRHI = ProxiedResource->SamplerStateRHI;
		DeferredPassSamplerStateRHI = ProxiedResource->DeferredPassSamplerStateRHI;
	}
	else
	{
		FStreamableTextureResource::InitRHI(RHICmdList);
	}
}
