// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

/*=============================================================================
	Texture3DResource.h: Implementation of FTexture3DResource used by streamable UVolumeTexture.
=============================================================================*/

#include "CoreMinimal.h"
#include "Rendering/StreamableTextureResource.h"
#include "Containers/ResourceArray.h"

class UVolumeTexture;

class FVolumeTextureBulkData : public FResourceBulkDataInterface
{
public:

	static const uint32 MALLOC_ALIGNMENT = 16;

	FVolumeTextureBulkData(int32 InFirstMipIdx)
	: FirstMipIdx(InFirstMipIdx)
	{
		FMemory::Memzero(MipData, sizeof(MipData));
		FMemory::Memzero(MipSize, sizeof(MipSize));
	}

	~FVolumeTextureBulkData()
	{ 
		Discard();
	}

	const void* GetResourceBulkData() const override
	{
		return MipData[FirstMipIdx];
	}

	void* GetResourceBulkData()
	{
		return MipData[FirstMipIdx];
	}

	uint32 GetResourceBulkDataSize() const override
	{

		return (uint32)MipSize[FirstMipIdx];
	}

	void Discard() override final;
	void MergeMips(int32 NumMips);

	void** GetMipData() { return MipData; }
	int64* GetMipSize() { return MipSize; }
	int32 GetFirstMipIdx() const { return FirstMipIdx; }

protected:

	void* MipData[MAX_TEXTURE_MIP_COUNT];
	int64 MipSize[MAX_TEXTURE_MIP_COUNT];
	int32 FirstMipIdx;
};


class FTexture3DResource : public FStreamableTextureResource
{
public: 

	FTexture3DResource(UVolumeTexture* InOwner, const FStreamableRenderResourceState& InState);

	/**
	 * Make this FTexture3DResource Proxy another one.
	 *
	 * @param InOwner             UVolumeTexture which this FTexture3DResource represents.
	 * @param InProxiedResource   The resource to proxy.
	 */
	FTexture3DResource(UVolumeTexture* InOwner, const FTexture3DResource* InProxiedResource);

	// Dynamic cast methods.
	virtual FTexture3DResource* GetTexture3DResource() override { return this; }
	// Dynamic cast methods (const).
	virtual const FTexture3DResource* GetTexture3DResource() const override { return this; }

	/** Returns the platform mip size for the given mip count. */
	virtual uint64 GetPlatformMipsSize(uint32 NumMips) const override;

	virtual void InitRHI(FRHICommandListBase& RHICmdList) override;
	virtual bool IsProxy() const override { return ProxiedResource != nullptr; }

private:

	void CreateTexture(FRHICommandListBase& RHICmdList) final override;
	void CreatePartiallyResidentTexture(FRHICommandListBase& RHICmdList) final override;

	/** Another resource being proxied by this one. */
	const FTexture3DResource* const ProxiedResource = nullptr;

protected:
	FVolumeTextureBulkData InitialData;

};
