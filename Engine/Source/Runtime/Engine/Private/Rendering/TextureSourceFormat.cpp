// Copyright Epic Games, Inc. All Rights Reserved.

#include "Engine/TextureDefines.h"
#include "PixelFormat.h"

//
//	Texture source format information.
//

FTextureSourceFormatInfo::FTextureSourceFormatInfo(ETextureSourceFormat InTextureSourceFormat, EPixelFormat InPixelFormat, const TCHAR* InName, int32 InNumComponents, int32 InBytesPerPixel)
	: TextureSourceFormat(InTextureSourceFormat)
	, PixelFormat(InPixelFormat)
	, Name(InName)
	, NumComponents(InNumComponents)
	, BytesPerPixel(InBytesPerPixel)
{}

FTextureSourceFormatInfo GTextureSourceFormats[TSF_MAX] =
{
	//						TextureSourceFormat		PixelFormat							Name						NumComponents	BytesPerPixel
	FTextureSourceFormatInfo(TSF_Invalid,			EPixelFormat::PF_Unknown,			TEXT("Invalid"),			0,				0),
	FTextureSourceFormatInfo(TSF_G8,				EPixelFormat::PF_G8,				TEXT("G8"),					1,				1),
	FTextureSourceFormatInfo(TSF_BGRA8,				EPixelFormat::PF_B8G8R8A8,			TEXT("BGRA8"),				4,				4),
	FTextureSourceFormatInfo(TSF_BGRE8,				EPixelFormat::PF_B8G8R8A8,			TEXT("BGRE8"),				4,				4),
	FTextureSourceFormatInfo(TSF_RGBA16,			EPixelFormat::PF_R16G16B16A16_UINT,	TEXT("RGBA16"),				4,				8),
	FTextureSourceFormatInfo(TSF_RGBA16F,			EPixelFormat::PF_A16B16G16R16,		TEXT("RGBA16F"),			4,				8),
	FTextureSourceFormatInfo(TSF_RGBA8_DEPRECATED,	EPixelFormat::PF_B8G8R8A8,			TEXT("RGBA8_DEPRECATED"),	4,				4),
	FTextureSourceFormatInfo(TSF_RGBE8_DEPRECATED,	EPixelFormat::PF_B8G8R8A8,			TEXT("RGBE8_DEPRECATED"),	4,				4),
	FTextureSourceFormatInfo(TSF_G16,				EPixelFormat::PF_G16,				TEXT("G16"),				1,				2),
	FTextureSourceFormatInfo(TSF_RGBA32F,			EPixelFormat::PF_A32B32G32R32F,		TEXT("RGBA32F"),			4,				16),
	FTextureSourceFormatInfo(TSF_R16F,				EPixelFormat::PF_R16F,				TEXT("R16F"),				1,				2),
	FTextureSourceFormatInfo(TSF_R32F,				EPixelFormat::PF_R32_FLOAT,			TEXT("R32F"),				1,				4),
};
