// Copyright Epic Games, Inc. All Rights Reserved.

#include "Engine/ToonProfile.h"
#include "ToonProfileDefinitions.h"
#include "Engine/Texture2D.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "Math/Float16.h"
#include "Rendering/BurleyNormalizedSSS.h"
#include "EngineModule.h"
#include "RenderTargetPool.h"
#include "PixelShaderUtils.h"
#include "RenderingThread.h"
#include "Rendering/Texture2DResource.h"
#include "RenderGraphResources.h"
#include "RenderGraphBuilder.h"
#include "SubstrateDefinitions.h"
#include "ShaderCompilerCore.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ToonProfile)

DEFINE_LOG_CATEGORY_STATIC(LogToonProfile, Log, All);

///////////////////////////////////////////////////////////////////////////////////////////////////


static TAutoConsoleVariable<int32> CVarToonProfileResolution(
	TEXT("r.Substrate.ToonProfile.Resolution"),
	64,
	TEXT("The resolution of the toon profile texture.\n"),
	ECVF_RenderThreadSafe
);

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
static TAutoConsoleVariable<int32> CVarToonProfileForceUpdate(
	TEXT("r.Substrate.ToonProfile.ForceUpdate"),
	0,
	TEXT("0: Only update the toon profile as needed.\n")
	TEXT("1: Force to update the toon profile every frame for debugging.\n"),
	ECVF_Cheat | ECVF_RenderThreadSafe);
#endif

static uint32 GetToonProfileResolution()
{
	return FMath::Max(64, CVarToonProfileResolution.GetValueOnRenderThread());
}

static bool ForceUpdateToonProfile()
{
	static int32 LastToonProfileResolution = 0;
	const int32 NewToonProfileResolution = GetToonProfileResolution();
	if (LastToonProfileResolution != NewToonProfileResolution)
	{
		LastToonProfileResolution = NewToonProfileResolution;
		return true;
	}

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	return (CVarToonProfileForceUpdate.GetValueOnAnyThread() == 1);
#else
	return false;
#endif
}

///////////////////////////////////////////////////////////////////////////////////////////////////

class FToonProfileUpdateAtlasTileCS : public FGlobalShader
{
public:
	DECLARE_GLOBAL_SHADER(FToonProfileUpdateAtlasTileCS);
	SHADER_USE_PARAMETER_STRUCT(FToonProfileUpdateAtlasTileCS, FGlobalShader);

	using FPermutationDomain = TShaderPermutationDomain<>;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER(FIntPoint, ToonProfileTextureResolution)
		SHADER_PARAMETER(uint32, ToonProfileIndex)
		SHADER_PARAMETER(float, DiffuseRampOffsetStrength)
		SHADER_PARAMETER(float, DiffuseRampOffsetSize)
		SHADER_PARAMETER(float, SpecularRampOffsetStrength)
		SHADER_PARAMETER(float, SpecularRampOffsetSize)
		SHADER_PARAMETER(float, ShadowHatchingPatternSize)
		SHADER_PARAMETER(float, ShadowHatchingPatternStrength)
		SHADER_PARAMETER(float, DiffuseIndirectScale)
		SHADER_PARAMETER(float, SpecularIndirectScale)
		SHADER_PARAMETER(float, bDiffuseRampIncludeShadow)
		SHADER_PARAMETER(float, SpecularIndirectRampRepetition)

		SHADER_PARAMETER(float, ToonProfileShadowExtinctionCoefficientInInvCentimeter)

		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2DArray, ToonProfileTexture)

		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, DiffuseRampOffsetTexture)
		SHADER_PARAMETER_SAMPLER(SamplerState, DiffuseRampOffsetTextureSampler)

		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, SpecularRampOffsetTexture)
		SHADER_PARAMETER_SAMPLER(SamplerState, SpecularRampOffsetTextureSampler)

		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, ShadowHatchingPatternTexture)
		SHADER_PARAMETER_SAMPLER(SamplerState, ShadowHatchingPatternTextureSampler)

		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer, DiffuseRampBufferSRV)
		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer, SpecularRampBufferSRV)
		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer, SpecularIndirectRampBufferSRV)
		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer, ShadowRampBufferSRV)
	END_SHADER_PARAMETER_STRUCT()

	static int32 GetThreadGroupSize()
	{
		return 8;
	}

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return Substrate::IsToonProfileEnabled();
	}
	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("THREAD_GROUP_COUNT"), GetThreadGroupSize());
	}
};

IMPLEMENT_GLOBAL_SHADER(FToonProfileUpdateAtlasTileCS, "/Engine/Private/ToonProfile.usf", "ToonProfileUpdateAtlasTileCS", SF_Compute);

///////////////////////////////////////////////////////////////////////////////////////////////////
// FToonProfileTextureManager

struct FToonProfileIdQuery
{
	bool bFound = false;		// default to not found
	uint32 AllocationId = 0;	// default entry
};

// render thread
class FToonProfileTextureManager : public FRenderResource
{
public:
	// constructor
	FToonProfileTextureManager();

	// destructor
	~FToonProfileTextureManager();


	// convenience, can be optimized 
	// @param Profile must not be 0, game thread pointer, do not dereference, only for comparison
	int32 AddOrUpdateProfile(const UToonProfile* InProfile, const FGuid& InGuid, const FToonProfileStruct InSettings);

	// O(n) n is a small number
	// @param InProfile must not be 0, game thread pointer, do not dereference, only for comparison
	// @return AllocationId INDEX_NONE: no allocation, should be deallocated with DeallocateToonProfile()
	int32 AddProfile(const UToonProfile* InProfile, const FGuid& InGuid, const FToonProfileStruct Settings);

	// O(n) to find the element, n is the SSProfile count and usually quite small
	void RemoveProfile(const UToonProfile* InProfile);

	// @param InProfile must not be 0, game thread pointer, do not dereference, only for comparison
	void UpdateProfile(const UToonProfile* InProfile, const FToonProfileStruct InSettings);

	// @param InProfile must not be 0, game thread pointer, do not dereference, only for comparison
	void UpdateProfile(FToonProfileIdQuery ToonProfileIdQuery, const FToonProfileStruct Settings);

	// return the parameter name for a given profile
	FName GetParameterName(const UToonProfile* InProfile) const;

	// @return can be nullptr if there is no ToonProfile
	IPooledRenderTarget* GetAtlasTexture(FRDGBuilder& GraphBuilder, EShaderPlatform ShaderPlatform);
	IPooledRenderTarget* GetAtlasTexture();

	//~ Begin FRenderResource Interface.
	/**
		* Release textures when device is lost/destroyed.
		*/
	virtual void ReleaseRHI() override;

	// @param InProfile must not be 0, game thread pointer, do not dereference, only for comparison
	// @return INDEX_NONE if not found
	FToonProfileIdQuery FindAllocationId(const UToonProfile* InProfile) const;

private:

	struct FToonProfileEntry
	{
		FToonProfileStruct Settings;
		const UToonProfile* Profile = nullptr; // Game thread pointer! Do not dereference, only for comparison.
		FName ParameterName;
		FIntPoint DiffuseRampOffsetTextureCachedResolution = FIntPoint::ZeroValue;
		FIntPoint SpecularRampOffsetTextureCachedResolution = FIntPoint::ZeroValue;
		FIntPoint ShadowHatchingPatternTextureCachedResolution = FIntPoint::ZeroValue;
	};
	TArray<FToonProfileEntry> ToonProfileEntries;
	const FTextureReference* Texture = nullptr;
};

// Global resources - lives on the render thread
TGlobalResource<FToonProfileTextureManager> GToonProfileTextureManager;

// ToonProfile atlas storing several texture profiles or 0 if there is no user
static TRefCountPtr<IPooledRenderTarget> GToonProfileTextureAtlas;

FToonProfileTextureManager::FToonProfileTextureManager()
{
	check(IsInGameThread());

	// add element 0, it is used as default profile
	FToonProfileEntry& Entry = ToonProfileEntries.AddDefaulted_GetRef();
	Entry.Settings = FToonProfileStruct();
	Entry.Profile = nullptr;
}

FToonProfileTextureManager::~FToonProfileTextureManager()
{
}

int32 FToonProfileTextureManager::AddOrUpdateProfile(const UToonProfile* InProfile, const FGuid& Guid, const FToonProfileStruct InSettings)
{
	check(InProfile);

	FToonProfileIdQuery ToonProfileIdQuery = FindAllocationId(InProfile);
		
	if (ToonProfileIdQuery.bFound)
	{
		UpdateProfile(ToonProfileIdQuery, InSettings);
		return ToonProfileIdQuery.AllocationId;
	}

	return AddProfile(InProfile, Guid, InSettings);
}

int32 FToonProfileTextureManager::AddProfile(const UToonProfile* InProfile, const FGuid& Guid, const FToonProfileStruct InSettings)
{
	check(InProfile);
	check(!FindAllocationId(InProfile).bFound);

	FToonProfileIdQuery ToonProfileIdQuery;
	{
		for (int32 i = 1; i < ToonProfileEntries.Num(); ++i)
		{
			if (ToonProfileEntries[i].Profile == nullptr)
			{
				ToonProfileIdQuery.bFound = true;
				ToonProfileIdQuery.AllocationId = i;

				FToonProfileEntry& Entry= ToonProfileEntries[ToonProfileIdQuery.AllocationId];
				Entry.Profile 		= InProfile;
				Entry.ParameterName = FName(TEXT("__ToonProfile") + Guid.ToString());
				break;
			}
		}

		if (!ToonProfileIdQuery.bFound)
		{
			ToonProfileIdQuery.bFound = true;
			ToonProfileIdQuery.AllocationId = ToonProfileEntries.Num();

			FToonProfileEntry& Entry= ToonProfileEntries.AddDefaulted_GetRef();
			Entry.Profile 		= InProfile;
			Entry.ParameterName = FName(TEXT("__ToonProfile") + Guid.ToString());
		}
	}

	UpdateProfile(ToonProfileIdQuery, InSettings);

	return ToonProfileIdQuery.AllocationId;
}

void FToonProfileTextureManager::RemoveProfile(const UToonProfile* InProfile)
{
	FToonProfileIdQuery ToonProfileIdQuery = FindAllocationId(InProfile);

	if (ToonProfileIdQuery.bFound)
	{
		// >0 as 0 is used as default profile which should never be removed
		check(ToonProfileIdQuery.AllocationId > 0);
		check(ToonProfileEntries[ToonProfileIdQuery.AllocationId].Profile == InProfile);

		// make it available for reuse
		ToonProfileEntries[ToonProfileIdQuery.AllocationId].Profile = nullptr;
		ToonProfileEntries[ToonProfileIdQuery.AllocationId].Settings.Invalidate();
		ToonProfileEntries[ToonProfileIdQuery.AllocationId].ParameterName = FName();
	}
}

void FToonProfileTextureManager::UpdateProfile(const UToonProfile* InProfile, const FToonProfileStruct InSettings) 
{ 
	UpdateProfile(FindAllocationId(InProfile), InSettings); 
}

void FToonProfileTextureManager::UpdateProfile(FToonProfileIdQuery ToonProfileIdQuery, const FToonProfileStruct Settings)
{
	check(IsInRenderingThread());

	if (ToonProfileIdQuery.bFound)
	{
		check(ToonProfileIdQuery.AllocationId < uint32(ToonProfileEntries.Num()));
		ToonProfileEntries[ToonProfileIdQuery.AllocationId].Settings = Settings;
		GToonProfileTextureAtlas.SafeRelease();
	}
}

FName FToonProfileTextureManager::GetParameterName(const UToonProfile* InProfile) const
{
	FToonProfileIdQuery ToonProfileIdQuery = FindAllocationId(InProfile);
	if (ToonProfileIdQuery.bFound)
	{
		// make it available for reuse
		return ToonProfileEntries[ToonProfileIdQuery.AllocationId].ParameterName;
	}
	return FName();
}

IPooledRenderTarget* FToonProfileTextureManager::GetAtlasTexture()
{
	return GToonProfileTextureAtlas;
}

IPooledRenderTarget* FToonProfileTextureManager::GetAtlasTexture(FRDGBuilder& GraphBuilder, EShaderPlatform ShaderPlatform)
{
	if (!Substrate::IsToonProfileEnabled())
	{
		return nullptr;
	}

	const uint32 ToonProfileCount = ToonProfileEntries.Num();
	check(ToonProfileCount);

	// Force update if cvars have changed.
	bool bForceUpdate = ForceUpdateToonProfile();

	// Force update if a texture has been streamed in and thus has changed. That is done by tracking change in resolution.
	for (uint32 LayerIt = 0; LayerIt < ToonProfileCount; ++LayerIt)
	{
		FToonProfileEntry& Entry = ToonProfileEntries[LayerIt];
		const FToonProfileStruct& Data = Entry.Settings;

		auto CheckIfTextureHasStreamedIn = [&](FTextureResource* TextureResource, FIntPoint& CachedResolution)
		{
			const uint32 TextureSizeX = TextureResource ? TextureResource->GetSizeX() : 0u;
			const uint32 TextureSizeY = TextureResource ? TextureResource->GetSizeY() : 0u;
			if (TextureSizeX != CachedResolution.X || TextureSizeY != CachedResolution.Y)
			{
				CachedResolution = FIntPoint(TextureSizeX, TextureSizeY);
				bForceUpdate = true;
			}
		};

		CheckIfTextureHasStreamedIn(Data.DiffuseRampOffsetTexture		? Data.DiffuseRampOffsetTexture->GetResource()		: nullptr, Entry.DiffuseRampOffsetTextureCachedResolution);
		CheckIfTextureHasStreamedIn(Data.SpecularRampOffsetTexture		? Data.SpecularRampOffsetTexture->GetResource()		: nullptr, Entry.SpecularRampOffsetTextureCachedResolution);
		CheckIfTextureHasStreamedIn(Data.ShadowHatchingPatternTexture	? Data.ShadowHatchingPatternTexture->GetResource()	: nullptr, Entry.ShadowHatchingPatternTextureCachedResolution);
	}

	if (bForceUpdate)
	{
		GToonProfileTextureAtlas.SafeRelease();
		GToonProfileTextureAtlas = nullptr;
	}

	if (GToonProfileTextureAtlas == nullptr)
	{
		FRHICommandListImmediate& RHICmdList = GraphBuilder.RHICmdList;

		const uint32 ToonProfileResolution = GetToonProfileResolution();

		RDG_EVENT_SCOPE(GraphBuilder, "ToonProfile");

		// See ToonProfileCommon.ush
		const uint32 ResolutionX = ToonProfileResolution;
		const uint32 ResolutionY = TOON_PROFILE_TEXTURES_Y + TOON_PROFILE_TEXTURE_COUNT * ToonProfileResolution; // row for ramps, parameters and textures.

		// 1. Create atlas texture
		const FIntPoint ToonProfileResolution2D = FIntPoint(ToonProfileResolution, ResolutionY);
		const EPixelFormat Format = PF_FloatRGBA;
		FPooledRenderTargetDesc Desc(FPooledRenderTargetDesc::Create2DArrayDesc(ToonProfileResolution2D, Format, FClearValueBinding::None, TexCreate_None, TexCreate_UAV | TexCreate_ShaderResource, false, ToonProfileCount));
		GRenderTargetPool.FindFreeElement(RHICmdList, Desc, GToonProfileTextureAtlas, TEXT("ToonProfileTexture"));
		const auto GlobalShaderMap = GetGlobalShaderMap(ShaderPlatform);
		FRDGTextureRef ToonProfileTexture = GraphBuilder.RegisterExternalTexture(GToonProfileTextureAtlas, TEXT("ToonProfileTexture"));
		
		// 2. Optionally clear profiles texture
		if (bForceUpdate)
		{
			AddClearUAVPass(GraphBuilder, GraphBuilder.CreateUAV(ToonProfileTexture), 0.f);
		}

		// 3. Fill in profiles
		FRDGTextureUAVRef ToonProfileUAVSkipBarrier = GraphBuilder.CreateUAV(ToonProfileTexture, ERDGUnorderedAccessViewFlags::SkipBarrier);
		for (uint32 LayerIt = 0; LayerIt < ToonProfileCount; ++LayerIt)
		{
			const FToonProfileStruct& Data = ToonProfileEntries[LayerIt].Settings;

			FTextureResource* DiffuseRampOffsetTextureResource = Data.DiffuseRampOffsetTexture ? Data.DiffuseRampOffsetTexture->GetResource() : nullptr;
			FRHITexture* DiffuseRampOffsetTexture2DRHI = DiffuseRampOffsetTextureResource ? DiffuseRampOffsetTextureResource->GetTexture2DRHI() : nullptr;
			bool bHasDiffuseRampOffsetTexture = DiffuseRampOffsetTexture2DRHI != nullptr;
			DiffuseRampOffsetTexture2DRHI = DiffuseRampOffsetTexture2DRHI ? DiffuseRampOffsetTexture2DRHI : GBlackTexture->TextureRHI->GetTexture2D();

			FTextureResource* SpecularRampOffsetTextureResource = Data.SpecularRampOffsetTexture ? Data.SpecularRampOffsetTexture->GetResource() : nullptr;
			FRHITexture* SpecularRampOffsetTexture2DRHI = SpecularRampOffsetTextureResource ? SpecularRampOffsetTextureResource->GetTexture2DRHI() : nullptr;
			bool bHasSpecularRampOffsetTexture = SpecularRampOffsetTexture2DRHI != nullptr;
			SpecularRampOffsetTexture2DRHI = SpecularRampOffsetTexture2DRHI ? SpecularRampOffsetTexture2DRHI : GBlackTexture->TextureRHI->GetTexture2D();

			FTextureResource* ShadowHatchingPatternTextureResource = Data.ShadowHatchingPatternTexture ? Data.ShadowHatchingPatternTexture->GetResource() : nullptr;
			FRHITexture* ShadowHatchingPatternTexture2DRHI = ShadowHatchingPatternTextureResource ? ShadowHatchingPatternTextureResource->GetTexture2DRHI() : nullptr;
			bool bHasShadowHatchingPatternTexture = ShadowHatchingPatternTexture2DRHI != nullptr;
			ShadowHatchingPatternTexture2DRHI = ShadowHatchingPatternTexture2DRHI ? ShadowHatchingPatternTexture2DRHI : GBlackTexture->TextureRHI->GetTexture2D();
			
			FToonProfileUpdateAtlasTileCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FToonProfileUpdateAtlasTileCS::FParameters>();
			PassParameters->ToonProfileTexture = ToonProfileUAVSkipBarrier;
			PassParameters->ToonProfileTextureResolution = ToonProfileTexture->Desc.Extent;
			PassParameters->ToonProfileIndex = LayerIt;
			PassParameters->DiffuseRampOffsetStrength = bHasDiffuseRampOffsetTexture ? Data.DiffuseRampOffsetStrength : 0.0f;
			PassParameters->DiffuseRampOffsetSize = Data.DiffuseRampOffsetSize;
			PassParameters->SpecularRampOffsetStrength = bHasSpecularRampOffsetTexture ? Data.SpecularRampOffsetStrength : 0.0f;
			PassParameters->SpecularRampOffsetSize = Data.SpecularRampOffsetSize;
			PassParameters->ToonProfileShadowExtinctionCoefficientInInvCentimeter = Data.GetShadowExtinctionCoefficientInInvCentimeter();
			PassParameters->ShadowHatchingPatternSize = Data.ShadowHatchingPatternSize;
			PassParameters->ShadowHatchingPatternStrength = bHasShadowHatchingPatternTexture ? Data.ShadowHatchingPatternStrength : 0.0f;

			PassParameters->DiffuseIndirectScale = Data.DiffuseIndirectScale;
			PassParameters->SpecularIndirectScale = Data.SpecularIndirectScale;
			PassParameters->bDiffuseRampIncludeShadow = Data.bDiffuseRampIncludeShadow ? 1.0f : 0.0f;
			PassParameters->SpecularIndirectRampRepetition = Data.SpecularIndirectRampRepetition;

			auto CreateBufferFromColorCurve = [&](const FRuntimeCurveLinearColor& ColorCurve)
			{
					const uint32 ComponentsPerElement = 4;
				const uint32 BytesPerComponent = sizeof(FFloat16);
				const uint32 BytesPerElement = ComponentsPerElement * BytesPerComponent;

				TArray<FFloat16> ColorCurveData;
				ColorCurveData.SetNum(ToonProfileResolution * ComponentsPerElement);
				for (uint32 It = 0; It < ToonProfileResolution; ++It)
				{
					FLinearColor CurveColorSample = ColorCurve.GetLinearColorValue(float(It + 0.5f) / ToonProfileResolution);
					ColorCurveData[It * 4 + 0] = CurveColorSample.R;
					ColorCurveData[It * 4 + 1] = CurveColorSample.G;
					ColorCurveData[It * 4 + 2] = CurveColorSample.B;
					ColorCurveData[It * 4 + 3] = CurveColorSample.A;
				}

				FRDGBufferRef ColorCurveBuffer = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateBufferDesc(BytesPerElement, ToonProfileResolution), TEXT("ColorCurveBuffer"), ERDGBufferFlags::MultiFrame);
				GraphBuilder.QueueBufferUpload(ColorCurveBuffer, ColorCurveData.GetData(), BytesPerComponent * ColorCurveData.Num(), ERDGInitialDataFlags::None);

				return ColorCurveBuffer;
			};
			auto CreateBufferFromFloatCurve = [&](const FRuntimeFloatCurve& FloatCurve)
				{
					const uint32 ComponentsPerElement = 4;
					const uint32 BytesPerComponent = sizeof(FFloat16);
					const uint32 BytesPerElement = ComponentsPerElement * BytesPerComponent;

					TArray<FFloat16> ColorCurveData;
					ColorCurveData.SetNum(ToonProfileResolution * ComponentsPerElement);
					for (uint32 It = 0; It < ToonProfileResolution; ++It)
					{
						float CurveSample = FloatCurve.GetRichCurveConst()->Eval(float(It + 0.5f) / ToonProfileResolution);
						ColorCurveData[It * 4 + 0] = CurveSample;
						ColorCurveData[It * 4 + 1] = CurveSample;
						ColorCurveData[It * 4 + 2] = CurveSample;
						ColorCurveData[It * 4 + 3] = CurveSample;
					}

					FRDGBufferRef ColorCurveBuffer = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateBufferDesc(BytesPerElement, ToonProfileResolution), TEXT("ColorCurveBuffer"), ERDGBufferFlags::MultiFrame);
					GraphBuilder.QueueBufferUpload(ColorCurveBuffer, ColorCurveData.GetData(), BytesPerComponent * ColorCurveData.Num(), ERDGInitialDataFlags::None);

					return ColorCurveBuffer;
				};

			FRDGBufferRef DiffuseRampBuffer = CreateBufferFromColorCurve(Data.DiffuseRamp);
			PassParameters->DiffuseRampBufferSRV = GraphBuilder.CreateSRV(DiffuseRampBuffer, Format);

			FRDGBufferRef SpecularRampBuffer = CreateBufferFromColorCurve(Data.SpecularRamp);
			PassParameters->SpecularRampBufferSRV = GraphBuilder.CreateSRV(SpecularRampBuffer, Format);

			FRDGBufferRef SpecularIndirectRampBuffer = CreateBufferFromFloatCurve(Data.SpecularIndirectRamp);
			PassParameters->SpecularIndirectRampBufferSRV = GraphBuilder.CreateSRV(SpecularIndirectRampBuffer, Format);

			FRDGBufferRef ShadowRampBuffer = CreateBufferFromFloatCurve(Data.ShadowHatchingPatternDistributionRamp);
			PassParameters->ShadowRampBufferSRV = GraphBuilder.CreateSRV(ShadowRampBuffer, Format);

			PassParameters->DiffuseRampOffsetTexture = GraphBuilder.RegisterExternalTexture(CreateRenderTarget(DiffuseRampOffsetTexture2DRHI, TEXT("DiffuseRampOffsetTexture")));
			PassParameters->DiffuseRampOffsetTextureSampler = TStaticSamplerState<SF_Trilinear, AM_Wrap, AM_Wrap>::GetRHI();

			PassParameters->SpecularRampOffsetTexture = GraphBuilder.RegisterExternalTexture(CreateRenderTarget(SpecularRampOffsetTexture2DRHI, TEXT("SpecularRampOffsetTexture")));
			PassParameters->SpecularRampOffsetTextureSampler = TStaticSamplerState<SF_Trilinear, AM_Wrap, AM_Wrap>::GetRHI();

			PassParameters->ShadowHatchingPatternTexture = GraphBuilder.RegisterExternalTexture(CreateRenderTarget(ShadowHatchingPatternTexture2DRHI, TEXT("ShadowHatchingPattern")));
			PassParameters->ShadowHatchingPatternTextureSampler = TStaticSamplerState<SF_Trilinear, AM_Wrap, AM_Wrap>::GetRHI();

			FToonProfileUpdateAtlasTileCS::FPermutationDomain PermutationVector;
			TShaderMapRef<FToonProfileUpdateAtlasTileCS> Shader(GlobalShaderMap, PermutationVector);
			FComputeShaderUtils::AddPass(GraphBuilder, 
				RDG_EVENT_NAME("ToonProfile::UpdateAtlasTileCS"), 
				Shader, 
				PassParameters, 
				FIntVector(
					FMath::DivideAndRoundUp(PassParameters->ToonProfileTextureResolution.X, FToonProfileUpdateAtlasTileCS::GetThreadGroupSize()), 
					FMath::DivideAndRoundUp(PassParameters->ToonProfileTextureResolution.Y, FToonProfileUpdateAtlasTileCS::GetThreadGroupSize()), 
					1)
			);
		}

		// Transit texture to SRV for letting the non-RDG resource to be bound later by the various passes
		FRDGExternalAccessQueue ExternalAccessQueue;
		ExternalAccessQueue.Add(ToonProfileTexture);
		ExternalAccessQueue.Submit(GraphBuilder);
	}
	return GToonProfileTextureAtlas;
}

void FToonProfileTextureManager::ReleaseRHI()
{
	GToonProfileTextureAtlas.SafeRelease();
}

FToonProfileIdQuery FToonProfileTextureManager::FindAllocationId(const UToonProfile* InProfile) const
{
	FToonProfileIdQuery ToonProfileIdQuery;

	// we start at 1 because [0] is the default profile and always [0].Profile = 0 so we don't need to iterate that one
	for (int32 i = 1; i < ToonProfileEntries.Num(); ++i)
	{
		if (ToonProfileEntries[i].Profile == InProfile)
		{
			ToonProfileIdQuery.bFound = true;
			ToonProfileIdQuery.AllocationId = i;
			return ToonProfileIdQuery;
		}
	}
	return ToonProfileIdQuery;
}

///////////////////////////////////////////////////////////////////////////////////////////////////
// FToonProfileStruct

FToonProfileStruct::FToonProfileStruct()
{
	auto AddPointToColorCurve = [](FRuntimeCurveLinearColor& LinearColorCurve, float Time, float Value)
	{
		LinearColorCurve.ColorCurves[0].AddKey(Time, Value);
		LinearColorCurve.ColorCurves[1].AddKey(Time, Value);
		LinearColorCurve.ColorCurves[2].AddKey(Time, Value);
	};
	auto AddPointToFloatCurve = [](FRuntimeFloatCurve& FloatCurve, float Time, float Value)
		{
			FloatCurve.EditorCurveData.AddKey(Time, Value);
		};

	AddPointToColorCurve(DiffuseRamp, 0.00f, 0.00f);
	AddPointToColorCurve(DiffuseRamp, 0.25f, 0.00f);
	AddPointToColorCurve(DiffuseRamp, 0.26f, 0.25f);
	AddPointToColorCurve(DiffuseRamp, 0.50f, 0.25f);
	AddPointToColorCurve(DiffuseRamp, 0.51f, 0.50f);
	AddPointToColorCurve(DiffuseRamp, 0.75f, 0.50f);
	AddPointToColorCurve(DiffuseRamp, 0.76f, 0.75f);
	AddPointToColorCurve(DiffuseRamp, 0.99f, 0.75f);
	AddPointToColorCurve(DiffuseRamp, 1.00f, 1.00f);

	AddPointToColorCurve(SpecularRamp, 0.00f, 0.0f);
	AddPointToColorCurve(SpecularRamp, 0.50f, 0.0f);
	AddPointToColorCurve(SpecularRamp, 0.51f, 1.0f);
	AddPointToColorCurve(SpecularRamp, 1.00f, 1.0f);

	AddPointToFloatCurve(SpecularIndirectRamp, 0.0f, 0.0f);
	AddPointToFloatCurve(SpecularIndirectRamp, 1.0f, 1.0f);

	DiffuseRampOffsetTexture = nullptr;
	DiffuseRampOffsetStrength = 0.0f;	// By default, skip.
	DiffuseRampOffsetSize = 1.0f;

	SpecularRampOffsetTexture = nullptr;
	SpecularRampOffsetStrength = 0.0f;	// By default, skip.
	SpecularRampOffsetSize = 1.0f;

	ShadowExtinctionCoefficient = 10.0f;	// Default extinction coefficients that work well.

	ShadowHatchingPatternTexture = nullptr;
	ShadowHatchingPatternSize = 1.0f;
	ShadowHatchingPatternStrength = 1.0f;

	DiffuseIndirectScale = 1.0f;
	SpecularIndirectScale = 1.0f;
	SpecularIndirectRampRepetition = 0.0f;

	bDiffuseRampIncludeShadow = 0;

	AddPointToFloatCurve(ShadowHatchingPatternDistributionRamp, 0.00, 0.0);
	AddPointToFloatCurve(ShadowHatchingPatternDistributionRamp, 1.00, 1.0);
}

///////////////////////////////////////////////////////////////////////////////////////////////////
// UToonProfile

UToonProfile::UToonProfile(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

void UToonProfile::BeginDestroy()
{
	UToonProfile* Ref = this;
	ENQUEUE_RENDER_COMMAND(RemoveToonProfile)(
	[Ref](FRHICommandList& RHICmdList)
	{
		GToonProfileTextureManager.RemoveProfile(Ref);
	});

	Super::BeginDestroy();
}

#if WITH_EDITOR
void UToonProfile::PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	const FToonProfileStruct LocalSettings = this->Settings;
	UToonProfile* LocalProfile = this;
	GetRendererModule().InvalidatePathTracedOutput();

	ENQUEUE_RENDER_COMMAND(UpdateToonProfile)(
	[LocalSettings, LocalProfile](FRHICommandListImmediate& RHICmdList)
	{
		// any changes to the setting require an update of the texture
		GToonProfileTextureManager.UpdateProfile(LocalProfile, LocalSettings);
	});
}

bool UToonProfile::CanEditChange(const FProperty* InProperty) const
{
	FString PropertyName = InProperty->GetName();

	if (PropertyName == GET_MEMBER_NAME_STRING_CHECKED(FToonProfileStruct, DiffuseRampOffsetStrength)
		|| PropertyName == GET_MEMBER_NAME_STRING_CHECKED(FToonProfileStruct, DiffuseRampOffsetSize))
	{
		return Settings.DiffuseRampOffsetTexture != nullptr;
	}
	
	if (PropertyName == GET_MEMBER_NAME_STRING_CHECKED(FToonProfileStruct, SpecularRampOffsetStrength)
		|| PropertyName == GET_MEMBER_NAME_STRING_CHECKED(FToonProfileStruct, SpecularRampOffsetSize))
	{
		return Settings.SpecularRampOffsetTexture != nullptr;
	}
	
	if (PropertyName == GET_MEMBER_NAME_STRING_CHECKED(FToonProfileStruct, ShadowHatchingPatternSize)
		|| PropertyName == GET_MEMBER_NAME_STRING_CHECKED(FToonProfileStruct, ShadowHatchingPatternStrength))
	{
		return Settings.ShadowHatchingPatternTexture != nullptr;
	}
	
	if (PropertyName == GET_MEMBER_NAME_STRING_CHECKED(FToonProfileStruct, SpecularIndirectRamp)
		|| PropertyName == GET_MEMBER_NAME_STRING_CHECKED(FToonProfileStruct, SpecularIndirectRampRepetition))
	{
		static IConsoleVariable* CVar = IConsoleManager::Get().FindConsoleVariable(TEXT("r.Substrate.Experimental.ToonReflectionQuantizationEnabled"));
		return CVar && CVar->GetInt() > 0;
	}

	return true;
}
#endif // WITH_EDITOR

void UToonProfile::PostDuplicate(EDuplicateMode::Type DuplicateMode)
{
	Super::PostDuplicate(DuplicateMode);

	// When a Toon Profile asset is duplicated/copied pasted (e.g. from the asset browser), we want the guid to be regenerated.
	Guid = FGuid::NewGuid();
}

///////////////////////////////////////////////////////////////////////////////////////////////////
// Public API

namespace ToonProfile
{
	FName GetToonProfileParameterName(const UToonProfile* In)
	{
		return GToonProfileTextureManager.GetParameterName(In);
	}

	float GetToonProfileId(const UToonProfile* In)
	{
		// No profile specified means we use the default one (constant one)
		int32 AllocationId = 0;
		if (In)
		{
			// can be optimized (cached)
			AllocationId = GToonProfileTextureManager.FindAllocationId(In).AllocationId;
		}
		return AllocationId;
	}
	
	int32 AddOrUpdateProfile(const UToonProfile* InProfile, const FGuid& InGuid, const FToonProfileStruct InSettings)
	{
		return GToonProfileTextureManager.AddOrUpdateProfile(InProfile, InGuid, InSettings);
	}

	FRHITexture* GetToonProfileTextureAtlas()
	{
		return GToonProfileTextureAtlas ? GToonProfileTextureAtlas->GetRHI() : nullptr;
	}

	FRHITexture* GetToonProfileTextureAtlasWithFallback()
	{
		return GToonProfileTextureAtlas ? GToonProfileTextureAtlas->GetRHI() : static_cast<FRHITexture*>(GWhiteTexture->TextureRHI);
	}

	void UpdateToonProfileTextureAtlas(FRDGBuilder& GraphBuilder, EShaderPlatform ShaderPlatform)
	{
		GToonProfileTextureManager.GetAtlasTexture(GraphBuilder, ShaderPlatform);
	}

	
	static FName CreateToonProfileParameterName(const FGuid& InGuid)
	{
		return FName(TEXT("__ToonProfile") + InGuid.ToString());
	}

	FName CreateToonProfileParameterName(UToonProfile* InProfile)
	{
		return InProfile ? CreateToonProfileParameterName(InProfile->Guid) : FName();
	}
} // namespace ToonProfile
