// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	RepLayout.cpp: Unreal replication layout implementation.
=============================================================================*/

#include "Net/RepLayout.h"
#include "Containers/StaticBitArray.h"
#include "Misc/MemStack.h"

#include "Net/Core/NetBitArray.h"
#include "Net/Core/NetCoreModule.h"
#include "Net/Core/Misc/NetCVars.h"
#include "Net/Core/NetToken/NetTokenExportContext.h"
#include "Net/Core/PropertyConditions/RepChangedPropertyTracker.h"
#include "Net/Core/PushModel/PushModel.h"
#include "Net/Core/PushModel/PushModelMacros.h"
#include "Net/Core/PushModel/Types/PushModelPerNetDriverState.h"
#include "Net/Core/Trace/NetTrace.h"
#include "Net/Core/Trace/Private/NetTraceInternal.h"

#include "EngineStats.h"
#include "Engine/PackageMapClient.h"
#include "Engine/NetConnection.h"

#include "Net/Serialization/FastArraySerializer.h"
#include "Net/NetworkProfiler.h"
#include "Engine/ActorChannel.h"
#include "Misc/App.h"

#include "Net/NetworkGranularMemoryLogging.h"
#include "Serialization/ArchiveCountMem.h"

#include "Stats/StatsTrace.h"
#include "UObject/EnumProperty.h"
#include "Iris/IrisConfig.h"
#include "Net/Iris/ReplicationSystem/ReplicationSystemUtil.h"

DECLARE_CYCLE_STAT(TEXT("RepLayout InitFromObjectClass"), STAT_RepLayout_InitFromObjectClass, STATGROUP_Game);
DECLARE_CYCLE_STAT(TEXT("RepLayout DeltaSerializeFastArray"), STAT_RepLayout_DeltaSerializeFastArray, STATGROUP_Game);

// LogRepProperties is very spammy, and the logs are in a very hot code path,
// so prevent anything less than a warning from even being compiled in on
// test and shipping builds.
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
DEFINE_LOG_CATEGORY_STATIC(LogRepProperties, Warning, All);
DEFINE_LOG_CATEGORY_STATIC(LogRepPropertiesBackCompat, Warning, All);
DEFINE_LOG_CATEGORY_STATIC(LogRepCompares, Warning, All);
#else
DEFINE_LOG_CATEGORY_STATIC(LogRepProperties, Warning, Warning);
DEFINE_LOG_CATEGORY_STATIC(LogRepPropertiesBackCompat, Warning, Warning);
DEFINE_LOG_CATEGORY_STATIC(LogRepCompares, Warning, Warning);
#endif

int32 GDoPropertyChecksum = 0;
static FAutoConsoleVariableRef CVarDoPropertyChecksum(TEXT("net.DoPropertyChecksum"), GDoPropertyChecksum,
	TEXT("When true and ENABLE_PROPERTY_CHECKSUMS is defined, checksums of replicated properties are compared on client and server"));

int32 GDoReplicationContextString = 0;
static FAutoConsoleVariableRef CVarDoReplicationContextString(TEXT("net.ContextDebug"), GDoReplicationContextString,
	TEXT("Debugging option to set a context string during replication"));

int32 GNetSharedSerializedData = 1;
static FAutoConsoleVariableRef CVarNetShareSerializedData(TEXT("net.ShareSerializedData"), GNetSharedSerializedData,
	TEXT("If true, enable shared serialization system used by replication to reduce CPU usage when multiple clients need the same data"));

int32 GNetVerifyShareSerializedData = 0;
static FAutoConsoleVariableRef CVarNetVerifyShareSerializedData(TEXT("net.VerifyShareSerializedData"), GNetVerifyShareSerializedData,
	TEXT("Debug option to verify shared serialization data during replication"));

int32 LogSkippedRepNotifies = 0;
static FAutoConsoleVariable CVarLogSkippedRepNotifies(TEXT("Net.LogSkippedRepNotifies"), LogSkippedRepNotifies, 
	TEXT("Log when the networking code skips calling a repnotify clientside due to the property value not changing."), ECVF_Default);

int32 GUsePackedShadowBuffers = 1;
static FAutoConsoleVariableRef CVarUsePackedShadowBuffers(TEXT("Net.UsePackedShadowBuffers"), GUsePackedShadowBuffers,
	TEXT("When enabled, FRepLayout will generate shadow buffers that are packed with only the necessary NetProperties, instead of copying entire object state."));

int32 GShareShadowState = 1;
static FAutoConsoleVariableRef CVarShareShadowState(TEXT("net.ShareShadowState"), GShareShadowState,
	TEXT("If true, work done to compare properties will be shared across connections"));

int32 GShareInitialCompareState = 0;
static FAutoConsoleVariableRef CVarShareInitialCompareState(TEXT("net.ShareInitialCompareState"), GShareInitialCompareState,
	TEXT("If true and net.ShareShadowState is enabled, attempt to also share initial replication compares across connections."));

bool GbTrackNetSerializeObjectReferences = false;
static FAutoConsoleVariableRef CVarTrackNetSerializeObjectReferences(TEXT("net.TrackNetSerializeObjectReferences"), GbTrackNetSerializeObjectReferences, TEXT("If true, we will create small layouts for Net Serialize Structs if they have Object Properties. This can prevent some Shadow State GC crashes."));

bool GbWithArrayOnRepFix = false;
static FAutoConsoleVariableRef CVarWithArrayOnRepFix(TEXT("net.WithArrayOnRepFix"), GbWithArrayOnRepFix, TEXT("If true, attempt to prevent issues with Arrays not receiving OnRep calls until their size changes if their Archetypes have different values from instances in levels."));

bool GbWithObjectValidationCheck = false;
static FAutoConsoleVariableRef CVarWithObjectValidationCheck(TEXT("net.WithObjectValidationCheck"), GbWithObjectValidationCheck, TEXT("If true, will run object property validation on property storage."));

#if WITH_PUSH_VALIDATION_SUPPORT

static bool GbPushModelValidateProperties = false;
static FAutoConsoleVariableRef CVarPushModelValidateProperties(TEXT("net.PushModelValidateProperties"), GbPushModelValidateProperties,
	TEXT("When true, we will compare all push model properties and warn if they haven't been marked dirty properly."));

#else

constexpr bool GbPushModelValidateProperties = false;

#endif

namespace UE::Net::Private
{
	static bool bDeltaInitialFastArrayElements = false;
	static FAutoConsoleVariableRef CVarDeltaInitialFastArrayElements(TEXT("net.DeltaInitialFastArrayElements"), bDeltaInitialFastArrayElements, TEXT("If true, send delta struct changelists for initial fast array elements."));

	static bool bAlwaysUpdateGuidReferenceMapForNetSerializeObjectStruct = false;
	static FAutoConsoleVariableRef CVarAlwaysUpdateGuidReferenceMapForNetSerializeStruct(TEXT("net.AlwaysUpdateGuidReferenceMapForNetSerializeObjectStruct"), bAlwaysUpdateGuidReferenceMapForNetSerializeObjectStruct,
		TEXT("Requires net.TrackNetSerializeObjectReferences. If true, entries in the GuidReferenceMap for NetSerialize struct properties with object properties will always be updated, not just when the Guid changes or goes NULL. This should prevent issues with old property data being applied when an unmapped actor ref in the struct is mapped."));

} // end namespace UE::Net::Private

extern int32 GNumSharedSerializationHit;
extern int32 GNumSharedSerializationMiss;

/** 
* Helper method to allow us to instrument FBitArchive using FNetTraceCollector
* The rationale behind this variant being declared here is that it makes ugly assumption about FBitArchive when being used to collect stats either is a BitWriter or a BitReader
*/
inline uint32 GetBitStreamPositionForNetTrace(const FBitArchive& Stream) { return (uint32(Stream.IsError()) - 1U) & (Stream.IsSaving() ? (uint32)(static_cast<const FBitWriter*>(&Stream))->GetNumBits() : (uint32)(static_cast<const FBitReader*>(&Stream))->GetPosBits()); }

namespace UEPushModelPrivate
{
	class FPushModelPerNetDriverState;
}

namespace UE_RepLayout_Private
{
	template<typename OutputType, typename CommandType, typename BufferType>
	static TCopyQualifiersFromTo_T<BufferType, OutputType>*
	GetTypedProperty(const BufferType& Buffer, const CommandType& Cmd)
	{
		using ConstOrNotOutputType = TCopyQualifiersFromTo_T<BufferType, OutputType>;

		using BaseBufferType = typename TRemovePointer<typename TDecay<BufferType>::Type>::Type;

		static_assert(!TIsPointer<OutputType>::Value, "GetTypedProperty invalid OutputType!  Don't specify output as a pointer.");
		static_assert(std::is_same_v<uint8, BaseBufferType> || std::is_same_v<void, BaseBufferType>, "GetTypedProperty invalid BufferType! Only TRepDataBufferBase, void*, and uint8* are supported!");

		// TODO: Conditionally compilable runtime type validation.
		return reinterpret_cast<ConstOrNotOutputType *>(Buffer);
	}

	template<typename OutputType, typename CommandType, enum ERepDataBufferType BufferDataType, typename BufferUnderlyingType>
	static TCopyQualifiersFromTo_T<typename TRepDataBufferBase<BufferDataType, BufferUnderlyingType>::ConstOrNotVoid, OutputType>*
	GetTypedProperty(const TRepDataBufferBase<BufferDataType, BufferUnderlyingType>& Buffer, const CommandType& Cmd)
	{
		using ConstOrNotOutputType = TCopyQualifiersFromTo_T<typename TRepDataBufferBase<BufferDataType, BufferUnderlyingType>::ConstOrNotVoid, OutputType>;

		static_assert(!TIsPointer<OutputType>::Value, "GetTypedProperty invalid OutputType! Don't specify output as a pointer.");

		// TODO: Conditionally compilable runtime type validation.
		return reinterpret_cast<ConstOrNotOutputType *>((Buffer + Cmd).Data);
	}

	static void QueueRepNotifyForCustomDeltaProperty(
		FReceivingRepState* RESTRICT ReceivingRepState,
		FNetDeltaSerializeInfo& Params,
		FProperty* Property,
		uint32 StaticArrayIndex)
	{
		//@note: AddUniqueItem() here for static arrays since RepNotify() currently doesn't indicate index,
		//			so reporting the same property multiple times is not useful and wastes CPU
		//			were that changed, this should go back to AddItem() for efficiency
		// @todo UE - not checking if replicated value is changed from old.  Either fix or document, as may get multiple repnotifies of unacked properties.
		ReceivingRepState->RepNotifies.AddUnique(Property);

		UFunction* RepNotifyFunc = Params.Object->FindFunctionChecked(Property->RepNotifyFunc);

		if (RepNotifyFunc->NumParms > 0)
		{
			if (Property->ArrayDim != 1)
			{
				// For static arrays, we build the meta data here, but adding the Element index that was just read into the PropMetaData array.
				UE_LOGF(LogRepTraffic, Verbose, "Property %ls had ArrayDim: %d change", *Property->GetName(), StaticArrayIndex);

				// Property is multi dimensional, keep track of what elements changed
				TArray<uint8>& PropMetaData = ReceivingRepState->RepNotifyMetaData.FindOrAdd(Property);
				PropMetaData.Add(StaticArrayIndex);
			}
		}
	}

	static void WritePropertyHeaderAndPayload(
		UObject* Object,
		UClass* ObjectClass,
		FProperty* Property,
		UNetConnection* Connection,
		UActorChannel* OwningChannel,
		FNetFieldExportGroup* NetFieldExportGroup,
		FNetBitWriter& Bunch,
		FNetBitWriter& Payload)
	{
		// Get class network info cache.
		const FClassNetCache* ClassCache = Connection->Driver->NetCache->GetClassNetCache(ObjectClass);

		check(ClassCache);

		// Get the network friend property index to replicate
		const FFieldNetCache * FieldCache = ClassCache->GetFromField(Property);

		checkSlow(FieldCache);

		// Send property name and optional array index.
		check(FieldCache->FieldNetIndex <= ClassCache->GetMaxIndex());

		// WriteFieldHeaderAndPayload will return the total number of bits written.
		// So, we subtract out the Payload size to get the actual number of header bits.
		const int32 HeaderBits = static_cast<int64>(OwningChannel->WriteFieldHeaderAndPayload(Bunch, ClassCache, FieldCache, NetFieldExportGroup, Payload)) - Payload.GetNumBits();

		NETWORK_PROFILER(GNetworkProfiler.TrackWritePropertyHeader(Property, HeaderBits, nullptr));
	}

#if WITH_PUSH_MODEL
	const UEPushModelPrivate::FPushModelPerNetDriverHandle ConditionallyAddPushModelObject(const UObject* const Object, const TSharedRef<const FRepLayout>& InRepLayout)
	{
		const int32 NumReplicatedProperties = InRepLayout->GetNumParents();

		// Implement shared PushModelids/NetHandles for Iris and PushModel to avoid conflicts when we mix systems - JIRA: UE-158304
		const bool bAllowedToCreateHandles = UEPushModelPrivate::IsHandleCreationAllowed();

		if (UEPushModelPrivate::IsPushModelEnabled() && bAllowedToCreateHandles &&
			NumReplicatedProperties > 0 &&
			EnumHasAnyFlags(InRepLayout->GetFlags(), ERepLayoutFlags::FullPushSupport | ERepLayoutFlags::PartialPushSupport))
		{
			const FObjectKey ObjectKey(Object);
			return UEPushModelPrivate::AddPushModelObject(ObjectKey, NumReplicatedProperties);
		}

		return UEPushModelPrivate::FPushModelPerNetDriverHandle::MakeInvalidHandle();
	}

	void ConditionallyRemovePushModelObject(const UEPushModelPrivate::FPushModelPerNetDriverHandle Handle)
	{
		if (Handle.IsValid())
		{
			UEPushModelPrivate::RemovePushModelObject(Handle);
		}
	}

#endif

	UEPushModelPrivate::FPushModelPerNetDriverState* GetPerNetDriverState(const FRepChangelistState* ChangelistState)
	{
#if WITH_PUSH_MODEL
		const UEPushModelPrivate::FPushModelPerNetDriverHandle Handle = ChangelistState->GetPushModelObjectHandle();
		if (Handle.IsValid())
		{
			return UEPushModelPrivate::GetPerNetDriverState(Handle);
		}
#endif

		return nullptr;
	}

	static bool IsNetworkProfilerEnabled()
	{
		NETWORK_PROFILER(return true;);
		return false;
	}

	static bool IsNetworkProfilerComparisonTrackingEnabled()
	{
#if USE_NETWORK_PROFILER
		return GNetworkProfiler.IsComparisonTrackingEnabled();
#else
		return false;
#endif
	}

	bool ValidateArraySize(const int32 ArraySize, const FProperty* const Property)
	{
		if (UNLIKELY(TNumericLimits<uint16>::Max() <= ArraySize))
		{
			FString ErrorString = FString::Printf(TEXT("ValidateArraySize: Replicated arrays must be smaller than %d elements in size. ArraySize = (%d) Property = (%s)."),
				TNumericLimits<uint16>::Max(), ArraySize, *Property->GetPathName());

			// Keep these separate so that we always log the error even if we only fire the ensure once
			// or if ensures are disabled.
			UE_LOGF(LogRepTraffic, Error, "%ls", *ErrorString);

			ensureMsgf(false, TEXT("%s"), *ErrorString);
			return false;
		}

		return true;
	}	
}

//~ TODO: Consider moving the FastArray members into their own sub-struct to save memory for non fast array
//~			custom delta properties. Almost all Custom Delta properties now **are** Fast Arrays, so this
//~			probably doesn't matter much at the moment.

struct FLifetimeCustomDeltaProperty
{
	FLifetimeCustomDeltaProperty(const uint16 InPropertyRepIndex)
		: PropertyRepIndex(InPropertyRepIndex)
	{
	}

	FLifetimeCustomDeltaProperty(
		const uint16 InPropertyRepIndex,
		const int32 InFastArrayItemsCommand,
		const int32 InFastArrayNumber,
		const int32 InFastArrayDeltaFlagsOffset,
		const int32 InFastArrayArrayReplicationKeyOffset,
		const int32 InFastArrayItemReplicationIdOffset
	)
		: PropertyRepIndex(InPropertyRepIndex)
		, FastArrayItemsCommand(InFastArrayItemsCommand)
		, FastArrayNumber(InFastArrayNumber)
		, FastArrayDeltaFlagsOffset(InFastArrayDeltaFlagsOffset)
		, FastArrayArrayReplicationKeyOffset(InFastArrayArrayReplicationKeyOffset)
		, FastArrayItemReplicationIdOffset(InFastArrayItemReplicationIdOffset)
	{
	}

	/** The RepIndex of the corresponding Property. This can be used as an index into FRepLayout::Parents. */
	const uint16 PropertyRepIndex = INDEX_NONE;

	/** If this is a Fast Array Serializer property, this will be the command index for the Fast Array Item array. */
	const int32 FastArrayItemsCommand = INDEX_NONE;

	/**
	 * If this is a Fast Array Serializer property, this will be the instance number in the class.
	 * This is used to lookup Changelists.
	 */
	const int32 FastArrayNumber = INDEX_NONE;

	/**
	 * If this is a Fast Array Serializer property (and it is set up correctly for Delta Serialization), this will be an
	 * offset from to the property.
	 */
	const int32 FastArrayDeltaFlagsOffset = INDEX_NONE;

	/**
	 * If this is a Fast Array Serializer property (and it is set up correctly for Delta Serialization), this will be a pointer
	 * to the FFastArraySerializer::ArrayReplicationKey property.
	 */
	const int32 FastArrayArrayReplicationKeyOffset = INDEX_NONE;

	/**
	 * If this is a Fast Array Serializer property (and it is set up correctly for Delta Serialization), this will be a pointer
	 * to the FFastArraySerializerItem::ReplicationID property.
	 */
	const int32 FastArrayItemReplicationIdOffset = INDEX_NONE;

	const EFastArraySerializerDeltaFlags GetFastArrayDeltaFlags(void const * const FastArray) const
	{
		return GetRefFromOffsetAndMemory<EFastArraySerializerDeltaFlags>(FastArray, FastArrayDeltaFlagsOffset);
	}

	const int32 GetFastArrayArrayReplicationKey(void const * const FastArray) const
	{
		return GetRefFromOffsetAndMemory<int32>(FastArray, FastArrayArrayReplicationKeyOffset);
	}

	const int32 GetFastArrayItemReplicationID(void const * const FastArrayItem) const
	{
		return GetRefFromOffsetAndMemory<int32>(FastArrayItem, FastArrayItemReplicationIdOffset);
	}

	int32& GetFastArrayItemReplicationIDMutable(void* const FastArrayItem) const
	{
		return GetRefFromOffsetAndMemory<int32>(FastArrayItem, FastArrayItemReplicationIdOffset);
	}

private:

	template<typename Output>
	static Output& GetRefFromOffsetAndMemory(void* Memory, const int32 Offset)
	{
		checkSlow(Offset != INDEX_NONE);
		return *reinterpret_cast<Output*>(reinterpret_cast<uint8*>(Memory) + Offset);
	}

	template<typename Output>
	static const Output& GetRefFromOffsetAndMemory(void const * const Memory, const int32 Offset)
	{
		return GetRefFromOffsetAndMemory<Output>(const_cast<void*>(Memory), Offset);
	}
};

/**
 * Acceleration tracking which properties are custom delta.
 * This will ultimately replace the need for FObjectReplicator::LifetimeCustomDeltaProperties.
 */
struct FLifetimeCustomDeltaState
{
public:

	FLifetimeCustomDeltaState(int32 HighestCustomDeltaRepIndex)
	{
		check(HighestCustomDeltaRepIndex >= 0);
		LifetimeCustomDeltaIndexLookup.Init(static_cast<uint16>(INDEX_NONE), HighestCustomDeltaRepIndex + 1);
	}

	void CountBytes(FArchive& Ar) const
	{
		GRANULAR_NETWORK_MEMORY_TRACKING_INIT(Ar, "FLifetimeCustomDeltaState::CountBytes");

		GRANULAR_NETWORK_MEMORY_TRACKING_TRACK("LifetimeCustomDeltaProperties", LifetimeCustomDeltaProperties.CountBytes(Ar));
		GRANULAR_NETWORK_MEMORY_TRACKING_TRACK("LifetimeCustomDeltaIndexLookup", LifetimeCustomDeltaIndexLookup.CountBytes(Ar));
	}

	const uint16 GetNumCustomDeltaProperties() const
	{
		return LifetimeCustomDeltaProperties.Num();
	}

	const uint16 GetNumFastArrayProperties() const
	{
		return NumFastArrayProperties;
	}

	const FLifetimeCustomDeltaProperty& GetCustomDeltaProperty(const uint16 CustomDeltaIndex) const
	{
		return LifetimeCustomDeltaProperties[CustomDeltaIndex];
	}

	const uint16 GetCustomDeltaIndexFromPropertyRepIndex(const uint16 PropertyRepIndex) const
	{
		const uint16 CustomDeltaIndex = LifetimeCustomDeltaIndexLookup[PropertyRepIndex];
		check(static_cast<uint16>(INDEX_NONE) != CustomDeltaIndex);
		return CustomDeltaIndex;
	}

	void Add(FLifetimeCustomDeltaProperty&& ToAdd)
	{
		check(static_cast<uint16>(INDEX_NONE) == LifetimeCustomDeltaIndexLookup[ToAdd.PropertyRepIndex]);

		if (ToAdd.FastArrayNumber != INDEX_NONE)
		{
			++NumFastArrayProperties;
		}

		LifetimeCustomDeltaIndexLookup[ToAdd.PropertyRepIndex] = LifetimeCustomDeltaProperties.Num();
		LifetimeCustomDeltaProperties.Emplace(MoveTemp(ToAdd));
	}

	void CompactMemory()
	{
		LifetimeCustomDeltaProperties.Shrink();
		LifetimeCustomDeltaIndexLookup.Shrink();
	}

private:

	//~ Since there is only 1 RepLayout per class, and we will only create a FLifetimeCustomDeltaState for a RepLayout whose owning class
	//~ has Custom Delta Properties, using 2 arrays here seems like a good trade off for performance, memory, and convenience as opposed
	//~ to a TMap or TSortedMap.
	//~
	//~ Having just a map alone makes it harder for external code to iterate over custom delta properties without exposing these internal
	//~ classes.
	//~
	//~ However, maintaining just an array of CustomDeltaProperties makes it less efficient to perform lookups (we either need to keep
	//~ the list sorted or do linear searches).

	TArray<FLifetimeCustomDeltaProperty> LifetimeCustomDeltaProperties;
	TArray<uint16> LifetimeCustomDeltaIndexLookup;

	/** The number of valid FFastArraySerializer properties we found. */
	uint16 NumFastArrayProperties = 0;
};

//~ Some of this complexity could go away if we introduced a new Compare step to Custom Delta Serializers
//~ instead of just relying on the standard serialization stuff. That would be a bigger backwards compatibility
//~ risk, however.

struct FDeltaArrayHistoryItem
{
	/** The set of changelists by element ID.*/
	TMap<int32, TArray<uint16>> ChangelistByID;

	void Reset()
	{
		ChangelistByID.Empty();
	}

	void CountBytes(FArchive& Ar) const
	{
		GRANULAR_NETWORK_MEMORY_TRACKING_INIT(Ar, "FDeltaArrayHistoryItem::CountBytes");

		GRANULAR_NETWORK_MEMORY_TRACKING_TRACK("ChangelistByID",
			ChangelistByID.CountBytes(Ar);
			for (const auto& KVP : ChangelistByID)
			{
				KVP.Value.CountBytes(Ar);
			}
		);
	}
};

struct FDeltaArrayHistoryState
{
	/** The maximum number of individual changelists allowed.*/
	static const uint32 MAX_CHANGE_HISTORY = 32;

	/** Circular buffer of changelists. */
	TArray<FDeltaArrayHistoryItem> ChangeHistory;
	TBitArray<> ChangeHistoryUpdated;

	/** Copy of the IDToIndexMap from the array when we last sent it. */
	TMap<int32, int32> IDToIndexMap;

	/** The latest ArrayReplicationKey sent to any connection. */
	int32 ArrayReplicationKey = INDEX_NONE;

	/** Index in the buffer where changelist history starts (i.e., the Oldest changelist). */
	uint32 HistoryStart = 0;

	/** Index in the buffer where changelist history ends (i.e., the Newest changelist). */
	uint32 HistoryEnd = 0;

	void InitHistory()
	{
		if (ChangeHistory.Num() == 0)
		{
			ChangeHistory.AddDefaulted(MAX_CHANGE_HISTORY);
			ChangeHistoryUpdated.Init(false, MAX_CHANGE_HISTORY);
		}
	}

	void CountBytes(FArchive& Ar) const
	{
		GRANULAR_NETWORK_MEMORY_TRACKING_INIT(Ar, "FDeltaArrayHistoryState::CountBytes");

		GRANULAR_NETWORK_MEMORY_TRACKING_TRACK("IDToIndexMap", IDToIndexMap.CountBytes(Ar));

		GRANULAR_NETWORK_MEMORY_TRACKING_TRACK("ChangeHistory", ChangeHistory.CountBytes(Ar));

		GRANULAR_NETWORK_MEMORY_TRACKING_TRACK("ChangeHistoryElements",
			for (const FDeltaArrayHistoryItem& HistoryItem : ChangeHistory)
			{
				HistoryItem.CountBytes(Ar);
			}
		);

		GRANULAR_NETWORK_MEMORY_TRACKING_TRACK("ChangeHistoryUpdated", ChangeHistoryUpdated.CountBytes(Ar));
	}
};

struct FCustomDeltaChangelistState
{
	FCustomDeltaChangelistState(const int32 NumArrays)
	{
		ArrayStates.SetNum(NumArrays);
	}

	/** Last Replication Frame where we modified histories and caused compares to happen. */
	uint32 LastReplicationFrame = 0;

	/**
	 * An array tracking the last compared history of Arrays.
	 * Indices should match FLifetimeCustomDeltaProperty::FastArrayNumber.
	 */
	TArray<FDeltaArrayHistoryState> ArrayStates;

	void CountBytes(FArchive& Ar) const
	{
		GRANULAR_NETWORK_MEMORY_TRACKING_INIT(Ar, "FCustomDeltaChangelistState::CountBytes");

		GRANULAR_NETWORK_MEMORY_TRACKING_TRACK("ArrayStates",
			ArrayStates.CountBytes(Ar);
			for (const FDeltaArrayHistoryState& ArrayState : ArrayStates)
			{
				ArrayState.CountBytes(Ar);
			}
		);
	}
};

#define ENABLE_PROPERTY_CHECKSUMS

//#define SANITY_CHECK_MERGES

#define USE_CUSTOM_COMPARE

//#define ENABLE_SUPER_CHECKSUMS

#ifdef USE_CUSTOM_COMPARE
static FORCEINLINE bool CompareBool(
	const FRepLayoutCmd&	Cmd,
	const void* 			A,
	const void* 			B)
{
	return Cmd.Property->Identical(A, B);
}

static FORCEINLINE bool CompareObject(
	const FRepLayoutCmd&	Cmd,
	const void* 			A,
	const void* 			B)
{
	// Until FObjectPropertyBase::Identical is made safe for GC'd objects, we need to do it manually
	// This saves us from having to add referenced objects during GC
	const FObjectPropertyBase* ObjProperty = CastFieldChecked<FObjectPropertyBase>(Cmd.Property);

	// With remote object handles, use TObjectPtrs so that the objects don't have to be resolved.
	// Since that's not safe for GC'd objects, use raw pointers otherwise.
#if UE_WITH_REMOTE_OBJECT_HANDLE
	const TObjectPtr<UObject> ObjectA = ObjProperty->GetObjectPtrPropertyValue(A);
	const TObjectPtr<UObject> ObjectB = ObjProperty->GetObjectPtrPropertyValue(B);
#else
	const UObject* ObjectA = ObjProperty->GetObjectPropertyValue(A);
	const UObject* ObjectB = ObjProperty->GetObjectPropertyValue(B);
#endif

	return ObjectA == ObjectB;
}

static FORCEINLINE bool CompareSoftObject(
	const FRepLayoutCmd& Cmd,
	const void* A,
	const void* B)
{
	// FSoftObjectProperty::Identical will get the SoftObjectPath for each pointer, and compare the Path etc.
	// It should also handle null checks, and doesn't try to dereference the object, so is GC safe.
	return Cmd.Property->Identical(A, B);
}

static FORCEINLINE bool CompareWeakObject(
	const FRepLayoutCmd& Cmd,
	const void* A,
	const void* B)
{
	const FWeakObjectProperty* const WeakObjectProperty = CastFieldChecked<FWeakObjectProperty>(Cmd.Property);
	const FWeakObjectPtr ObjectA = WeakObjectProperty->GetPropertyValue(A);
	const FWeakObjectPtr ObjectB = WeakObjectProperty->GetPropertyValue(B);

	return ObjectA.HasSameIndexAndSerialNumber(ObjectB);
}

static FORCEINLINE bool CompareInterface(
	const FRepLayoutCmd& Cmd,
	const void* A,
	const void* B)
{
	return Cmd.Property->Identical(A, B);
}

static FORCEINLINE bool CompareNetSerializeStructWithObjectProperties(
	const TArray<FRepLayoutCmd>& Cmds,
	const TMap<FRepLayoutCmd*, TArray<FRepLayoutCmd>>& NetSerializeLayouts,
	const int32 CmdStart,
	const int32 CmdEnd,
	const void* A,
	const void* B);

template<typename T>
bool CompareValue(const T * A, const T * B)
{
	return *A == *B;
}

template<typename T>
bool CompareValue(const void* A, const void* B)
{
	return CompareValue((T*)A, (T*)B);
}

static FORCEINLINE bool PropertiesAreIdenticalNative(
	const FRepLayoutCmd& Cmd,
	const void* A,
	const void* B,
	const TMap<FRepLayoutCmd*, TArray<FRepLayoutCmd>>& NetSerializeLayouts)
{
	switch (Cmd.Type)
	{
		case ERepLayoutCmdType::PropertyBool:
			return CompareBool(Cmd, A, B);

		case ERepLayoutCmdType::PropertyNativeBool:
			return CompareValue<bool>(A, B);

		case ERepLayoutCmdType::PropertyByte:
			return CompareValue<uint8>(A, B);

		case ERepLayoutCmdType::PropertyFloat:
			return CompareValue<float>(A, B);

		case ERepLayoutCmdType::PropertyInt:
			return CompareValue<int32>(A, B);

		case ERepLayoutCmdType::PropertyName:
			return CompareValue<FName>(A, B);

		case ERepLayoutCmdType::PropertyObject:
			return CompareObject(Cmd, A, B);

		case ERepLayoutCmdType::PropertySoftObject:
			return CompareSoftObject(Cmd, A, B);

		case ERepLayoutCmdType::PropertyWeakObject:
			return CompareWeakObject(Cmd, A, B);

		case ERepLayoutCmdType::PropertyInterface:
			return CompareInterface(Cmd, A, B);

		case ERepLayoutCmdType::PropertyUInt32:
			return CompareValue<uint32>(A, B);

		case ERepLayoutCmdType::PropertyUInt64:
			return CompareValue<uint64>(A, B);

		case ERepLayoutCmdType::PropertyVector:
			return CompareValue<FVector>(A, B);

		case ERepLayoutCmdType::PropertyVector100:
			return CompareValue<FVector_NetQuantize100>(A, B);

		case ERepLayoutCmdType::PropertyVectorQ:
			return CompareValue<FVector_NetQuantize>(A, B);

		case ERepLayoutCmdType::PropertyVectorNormal:
			return CompareValue<FVector_NetQuantizeNormal>(A, B);

		case ERepLayoutCmdType::PropertyVector10:
			return CompareValue<FVector_NetQuantize10>(A, B);

		case ERepLayoutCmdType::PropertyPlane:
			return CompareValue<FPlane>(A, B);

		case ERepLayoutCmdType::PropertyRotator:
			return CompareValue<FRotator>(A, B);

		case ERepLayoutCmdType::PropertyNetId:
			return CompareValue<FUniqueNetIdRepl>(A, B);

		case ERepLayoutCmdType::RepMovement:
			return CompareValue<FRepMovement>(A, B);

		case ERepLayoutCmdType::PropertyString:
			return CompareValue<FString>(A, B);

		case ERepLayoutCmdType::NetSerializeStructWithObjectReferences:
			return CompareNetSerializeStructWithObjectProperties(NetSerializeLayouts.FindChecked(&Cmd), NetSerializeLayouts, 0, INDEX_NONE, A, B);

		case ERepLayoutCmdType::Property:
			return Cmd.Property->Identical(A, B);

		default: 
			UE_LOGF(LogRep, Fatal, "PropertiesAreIdentical: Unsupported type! %i (%ls)", (uint8)Cmd.Type, *Cmd.Property->GetName());
	}

	return false;
}

static FORCEINLINE bool CompareNetSerializeStructWithObjectProperties(
	const TArray<FRepLayoutCmd>& Cmds,
	const TMap<FRepLayoutCmd*, TArray<FRepLayoutCmd>>& NetSerializeLayouts,
	const int32 CmdStart,
	const int32 CmdEnd,
	const void* A,
	const void* B)
{
	const int32 RealCmdEnd = (CmdEnd == INDEX_NONE) ? Cmds.Num() : CmdEnd;
	for (int32 CmdIndex = 0; CmdIndex < RealCmdEnd; ++CmdIndex)
	{
		const FRepLayoutCmd& Cmd = Cmds[CmdIndex];
		check(Cmd.Type != ERepLayoutCmdType::Return);

		// These commands will use an offset from the start of the owning NetSerializeStruct.
		// So we can avoid the ShadowData / ObjectData stuff here and just use standard offsets.
		// This will work with packed or unpacked shadow buffers, because net serialize structs
		// aren't packed.

		UE_LOGF(LogRepCompares, VeryVerbose, "CompareNetSerializeStructWithObjectProperties: CmdIndex: %d CmdType: %ls Property: %ls", CmdIndex, LexToString(Cmd.Type), *GetNameSafe(Cmd.Property));

		if (ERepLayoutCmdType::DynamicArray == Cmd.Type)
		{
			FScriptArrayHelper AArray((FArrayProperty*)Cmd.Property, ((const uint8*)A + Cmd.Offset));
			FScriptArrayHelper BArray((FArrayProperty*)Cmd.Property, ((const uint8*)B + Cmd.Offset));

			if (AArray.Num() == BArray.Num())
			{
				for (int32 ArrayIndex = 0; ArrayIndex < AArray.Num(); ++ArrayIndex)
				{
					if (!CompareNetSerializeStructWithObjectProperties(Cmds, NetSerializeLayouts, CmdIndex + 1, Cmd.EndCmd - 1, AArray.GetRawPtr(ArrayIndex), BArray.GetRawPtr(ArrayIndex)))
					{
						return false;
					}
				}

				// The -1 to handle the ++ in the for loop
				CmdIndex = Cmd.EndCmd - 1;
			}
			else
			{
				return false;
			}
		}
		else if (!PropertiesAreIdenticalNative(Cmd, (const uint8*)A + Cmd.Offset, (const uint8*)B + Cmd.Offset, NetSerializeLayouts))
		{
			return false;
		}
	}

	return true;
}

static FORCEINLINE bool PropertiesAreIdentical(
	const FRepLayoutCmd& Cmd,
	const void* A,
	const void* B,
	const TMap<FRepLayoutCmd*, TArray<FRepLayoutCmd>>& NetSerializeLayouts)
{
	const bool bIsIdentical = PropertiesAreIdenticalNative(Cmd, A, B, NetSerializeLayouts);
#if 0
	// Sanity check result
	if (bIsIdentical != Cmd.Property->Identical(A, B))
	{
		UE_LOGF(LogRep, Fatal, "PropertiesAreIdentical: Result mismatch! (%ls)", *Cmd.Property->GetFullName());
	}
#endif
	return bIsIdentical;
}
#else
static FORCEINLINE bool PropertiesAreIdentical(
	const FRepLayoutCmd& Cmd,
	const void* A,
	const void* B,
	const TMap<FRepLayoutCmd*, TArray<FRepLayoutCmd>>& NetSerializeLayouts)
{
	return Cmd.Property->Identical(A, B);
}
#endif

static FORCEINLINE void StoreProperty(const FRepLayoutCmd& Cmd, void* A, const void* B)
{
	UE_LOGF(LogRepCompares, VeryVerbose, "StoreProperty: %ls", *GetNameSafe(Cmd.Property));

	if (GbWithObjectValidationCheck)
	{
		if (Cmd.Type == ERepLayoutCmdType::PropertyObject ||
			Cmd.Type == ERepLayoutCmdType::PropertyWeakObject ||
			Cmd.Type == ERepLayoutCmdType::PropertySoftObject)
		{
			if (const FObjectPropertyBase* ObjProperty = CastFieldChecked<FObjectPropertyBase>(Cmd.Property))
			{
				const UObject* AObjValue = ObjProperty->GetObjectPropertyValue(A);
				if (AObjValue && !IsValid(AObjValue))
				{
					UE_LOGF(LogRepCompares, Warning, "Destination Object %ls was not valid when storing for replication. StoreProperty: %ls", *GetNameSafe(AObjValue), *GetFullNameSafe(Cmd.Property));
				}

				const UObject* BObjValue = ObjProperty->GetObjectPropertyValue(B);
				if (BObjValue && !IsValid(BObjValue))
				{
					UE_LOGF(LogRepCompares, Warning, "Source Object %ls was not valid when storing for replication. StoreProperty: %ls", *GetNameSafe(BObjValue), *GetFullNameSafe(Cmd.Property));
				}
			}
		}
	}

	Cmd.Property->CopySingleValue(A, B);
}

static FORCEINLINE void SerializeGenericChecksum(FBitArchive& Ar)
{
	uint32 Checksum = 0xABADF00D;
	Ar << Checksum;
	check(Checksum == 0xABADF00D);
}

template<ERepDataBufferType DataType>
static void SerializeReadWritePropertyChecksum(
	const FRepLayoutCmd& Cmd,
	const int32 CurCmdIndex,
	const TConstRepDataBuffer<DataType> Data,
	FBitArchive& Ar)
{
	// Serialize various attributes that will mostly ensure we are working on the same property
	const uint32 NameHash = GetTypeHash(Cmd.Property->GetName());

	uint32 MarkerChecksum = 0;

	// Evolve the checksum over several values that will uniquely identity where we are and should be
	MarkerChecksum = FCrc::MemCrc_DEPRECATED(&NameHash, sizeof(NameHash), MarkerChecksum);
	MarkerChecksum = FCrc::MemCrc_DEPRECATED(&Cmd.Offset, sizeof(Cmd.Offset), MarkerChecksum);
	MarkerChecksum = FCrc::MemCrc_DEPRECATED(&CurCmdIndex, sizeof(CurCmdIndex), MarkerChecksum);

	const uint32 OriginalMarkerChecksum = MarkerChecksum;

	Ar << MarkerChecksum;

	if (MarkerChecksum != OriginalMarkerChecksum)
	{
		// This is fatal, as it means we are out of sync to the point we can't recover
		UE_LOGF(LogRep, Fatal, "SerializeReadWritePropertyChecksum: Property checksum marker failed! [%ls]", *Cmd.Property->GetFullName());
	}

	if (Cmd.Property->IsA(FObjectPropertyBase::StaticClass()))
	{
		// Can't handle checksums for objects right now
		// Need to resolve how to handle unmapped objects
		return;
	}

	// Now generate a checksum that guarantee that this property is in the exact state as the server
	// This will require NetSerializeItem to be deterministic, in and out
	// i.e, not only does NetSerializeItem need to write the same blob on the same input data, but
	//	it also needs to write the same blob it just read as well.
	FBitWriter Writer(0, true);

	Cmd.Property->NetSerializeItem(Writer, NULL, const_cast<uint8*>(Data.Data));

	if (Ar.IsSaving())
	{
		// If this is the server, do a read, and then another write so that we do exactly what the client will do, which will better ensure determinism 

		// We do this to force InitializeValue, DestroyValue etc to work on a single item
		const int32 OriginalDim = Cmd.Property->ArrayDim;
		Cmd.Property->ArrayDim = 1;

		TArray<uint8> TempPropMemory;
		TempPropMemory.AddZeroed(Cmd.Property->GetElementSize() + 4);
		uint32* Guard = (uint32*)&TempPropMemory[TempPropMemory.Num() - 4];
		const uint32 TAG_VALUE = 0xABADF00D;
		*Guard = TAG_VALUE;
		Cmd.Property->InitializeValue(TempPropMemory.GetData());
		check(*Guard == TAG_VALUE);

		// Read it back in and then write it out to produce what the client will produce
		FBitReader Reader(Writer.GetData(), Writer.GetNumBits());
		Cmd.Property->NetSerializeItem(Reader, NULL, TempPropMemory.GetData());
		check(Reader.AtEnd() && !Reader.IsError());
		check(*Guard == TAG_VALUE);

		// Write it back out for a final time
		Writer.Reset();

		Cmd.Property->NetSerializeItem(Writer, NULL, TempPropMemory.GetData());
		check(*Guard == TAG_VALUE);

		// Destroy temp memory
		Cmd.Property->DestroyValue(TempPropMemory.GetData());

		// Restore the static array size
		Cmd.Property->ArrayDim = OriginalDim;

		check(*Guard == TAG_VALUE);
	}

	uint32 PropertyChecksum = FCrc::MemCrc_DEPRECATED(Writer.GetData(), Writer.GetNumBytes());

	const uint32 OriginalPropertyChecksum = PropertyChecksum;

	Ar << PropertyChecksum;

	if (PropertyChecksum != OriginalPropertyChecksum)
	{
		// This is a warning, because for some reason, float rounding issues in the quantization functions cause this to return false positives
		UE_LOGF(LogRep, Warning, "Property checksum failed! [%ls]", *Cmd.Property->GetFullName());
	}
}

static uint32 GetRepLayoutCmdCompatibleChecksum(
	const FProperty*		Property,
	const UNetConnection*	ServerConnection,
	const uint32			StaticArrayIndex,
	const uint32			InChecksum)
{
	// Compatible checksums are only used for InternalAck connections
	if (ServerConnection && !ServerConnection->IsInternalAck())
	{
		return 0;
	}

	// Evolve checksum on name
	uint32 CompatibleChecksum = FCrc::StrCrc32(*Property->GetName().ToLower(), InChecksum);	
	
	// Evolve by property type
	const FObjectProperty* const ObjectPtrProperty = CastField<const FObjectProperty>(Property);

	FString CPPType = Property->GetCPPType(nullptr, EPropertyExportCPPFlags::CPPF_NoTObjectPtr).ToLower();
	CompatibleChecksum = FCrc::StrCrc32(*CPPType, CompatibleChecksum);
	
	// Evolve by StaticArrayIndex (to make all unrolled static array elements unique)
	if ((ServerConnection == nullptr) ||
		(ServerConnection->GetNetworkCustomVersion(FEngineNetworkCustomVersion::Guid) >= FEngineNetworkCustomVersion::RepCmdChecksumRemovePrintf))
	{
		CompatibleChecksum = FCrc::MemCrc32(&StaticArrayIndex, sizeof(StaticArrayIndex), CompatibleChecksum);
	}
	else
	{
		CompatibleChecksum = FCrc::StrCrc32(*FString::Printf(TEXT("%i"), StaticArrayIndex), CompatibleChecksum);
	}

	// Evolve by enum max value bits required
	if ((ServerConnection == nullptr) ||
		(ServerConnection->GetNetworkCustomVersion(FEngineNetworkCustomVersion::Guid) >= FEngineNetworkCustomVersion::EnumSerializationCompat))
	{
		if (const FEnumProperty* EnumProp = CastField<FEnumProperty>(Property))
		{
			const uint64 MaxBits = EnumProp->GetMaxNetSerializeBits();

			CompatibleChecksum = FCrc::MemCrc32(&MaxBits, sizeof(MaxBits), CompatibleChecksum);
		}
		else if (const FByteProperty* ByteProp = CastField<FByteProperty>(Property))
		{
			const uint64 MaxBits = ByteProp->GetMaxNetSerializeBits();

			CompatibleChecksum = FCrc::MemCrc32(&MaxBits, sizeof(MaxBits), CompatibleChecksum);
		}
	}

	return CompatibleChecksum;
}

#if (WITH_PUSH_MODEL)
struct FNetPrivatePushIdHelper
{
	static void SetNetPushID(UObject* InObject, const UEPushModelPrivate::FNetLegacyPushObjectId ObjectId)
	{
		const UEPushModelPrivate::FNetPushObjectId CurrentId(InObject->GetNetPushIdDynamic());
		if (CurrentId.GetLegacyPushObjectId() != ObjectId)
		{
			if (CurrentId.IsValid())
			{
				UE_LOGF(LogRep, Error, "SetNetPushID: %ls already has a push id. Existing ID = %ls, New ID = %ls",
					*InObject->GetPathName(),
					*UEPushModelPrivate::ToString(CurrentId),
					*UEPushModelPrivate::ToString(UEPushModelPrivate::FNetPushObjectId(ObjectId)));

				if (!UEPushModelPrivate::ValidateObjectIdReassignment(CurrentId.GetLegacyPushObjectId(), ObjectId))
				{
					return;
				}
			}

			FObjectNetPushIdHelper::SetNetPushIdDynamic(InObject, UEPushModelPrivate::FNetPushObjectId(ObjectId).GetValue());
		}
	}

	static void MarkPropertyDirty(const UObject* const InObject, const int32 RepIndex)
	{
		MARK_PROPERTY_DIRTY_UNSAFE(InObject, RepIndex);
	}
};
#endif // (WITH_PUSH_MODEL)

void FRepStateStaticBuffer::CountBytes(FArchive& Ar) const
{
	GRANULAR_NETWORK_MEMORY_TRACKING_INIT(Ar, "FRepStateStaticBuffer::CountBytes");

	// Unfortunately, this won't track Custom Serialize stucts or Custom Delta Serialize
	// structs.
	struct FCountBytesHelper
	{
		FCountBytesHelper(
			FArchiveCountMem& InAr,
			const FConstRepShadowDataBuffer InShadowData,
			const uint64 InTotalShadowMemory,
			const TArray<FRepParentCmd>& InParents,
			const TArray<FRepLayoutCmd>& InCmds)

			: Ar(InAr)
			, MainShadowData(InShadowData)
			, Parents(InParents)
			, Cmds(InCmds)
			, TotalShadowMemory(InTotalShadowMemory)
		{
		}

		void CountBytes()
		{
			uint64 NewMax = Ar.GetMax();
			uint64 OldMax = 0;
			uint64 OldShadowOffset = 0u;

			for (int32 ParentIndex = 0; ParentIndex < Parents.Num(); ++ParentIndex)
			{
				const FRepParentCmd& Parent = Parents[ParentIndex];

				OldMax = NewMax;

				CountBytes_Command(Parent, Parent.CmdStart, Parent.CmdEnd, MainShadowData);

				NewMax = Ar.GetMax();

				if (0 < Parent.RepNotifyNumParams ||
					(0 == Parent.RepNotifyNumParams && REPNOTIFY_OnChanged == Parent.RepNotifyCondition))
				{
					OnRepMemory += (NewMax - OldMax);
				}
				else
				{
					const uint64 NextShadowOffset = (ParentIndex < Parents.Num() - 1) ? Parents[ParentIndex + 1].ShadowOffset : TotalShadowMemory;

					NonRepMemory += (NewMax - OldMax);
					NonRepStaticMemory += NextShadowOffset - OldShadowOffset;
				}

				OldShadowOffset = Parent.ShadowOffset;
			}
		}

		void CountBytes_Command(const FRepParentCmd& Parent, const int32 CmdStart, const int32 CmdEnd, const FConstRepShadowDataBuffer ShadowData)
		{
			for (int32 CmdIndex = CmdStart; CmdIndex < CmdEnd; ++CmdIndex)
			{
				CountBytes_r(Parent, Cmds[CmdIndex], CmdIndex, ShadowData);
			}
		}

		void CountBytes_r(const FRepParentCmd& Parent, const FRepLayoutCmd& Cmd, int32& InCmdIndex, const FConstRepShadowDataBuffer ShadowData)
		{
			if (ERepLayoutCmdType::DynamicArray == Cmd.Type)
			{
				FScriptArray* Array = (FScriptArray*)(ShadowData + Cmd).Data;
				Array->CountBytes(Ar, Cmd.ElementSize);

				FConstRepShadowDataBuffer ShadowArrayData(Array->GetData());

				for (int32 i = 0; i < Array->Num(); ++i)
				{
					const int32 ArrayElementOffset = Cmd.ElementSize * i;
					CountBytes_Command(Parent, InCmdIndex + 1, Cmd.EndCmd - 1, ShadowArrayData + ArrayElementOffset);
				}

				InCmdIndex = Cmd.EndCmd - 1;
			}
			else if (ERepLayoutCmdType::PropertyString == Cmd.Type)
			{
				((FString const * const)(ShadowData + Cmd).Data)->CountBytes(Ar);
			}
		}

		FArchiveCountMem& Ar;
		const FConstRepShadowDataBuffer MainShadowData;
		const TArray<FRepParentCmd>& Parents;
		const TArray<FRepLayoutCmd>& Cmds;

		const uint64 TotalShadowMemory;
		uint64 OnRepMemory = 0;
		uint64 NonRepMemory = 0;
		uint64 NonRepStaticMemory = 0u;
	};

	FArchiveCountMem LocalAr(nullptr);
	Buffer.CountBytes(LocalAr);
	const uint64 StaticTotalMemory = LocalAr.GetMax();

	FCountBytesHelper CountBytesHelper(LocalAr, Buffer.GetData(), StaticTotalMemory, RepLayout->Parents, RepLayout->Cmds);

	// Buffers can be empty for ReceivingRepStates that are being tracked on servers.
	// TODO: Do we actually need to allocate receiving rep states on servers at all?
	if (Buffer.Num() > 0)
	{
		CountBytesHelper.CountBytes();
	}

	const uint64 StaticOnRepMemory = StaticTotalMemory - CountBytesHelper.NonRepStaticMemory;

	GRANULAR_NETWORK_MEMORY_TRACKING_TRACK("Static_OnRep", Ar.CountBytes(StaticTotalMemory, StaticTotalMemory));
	GRANULAR_NETWORK_MEMORY_TRACKING_TRACK("Static_NotOnRep", Ar.CountBytes(CountBytesHelper.NonRepStaticMemory, CountBytesHelper.NonRepStaticMemory));
	GRANULAR_NETWORK_MEMORY_TRACKING_TRACK("Dynamic_OnRep", Ar.CountBytes(CountBytesHelper.OnRepMemory, CountBytesHelper.OnRepMemory));
	GRANULAR_NETWORK_MEMORY_TRACKING_TRACK("Dynamic_NotOnRep", Ar.CountBytes(CountBytesHelper.NonRepMemory, CountBytesHelper.NonRepMemory));
}

FRepChangelistState::FRepChangelistState(
	const TSharedRef<const FRepLayout>& InRepLayout,
	const uint8* InSource,
	const UObject* InRepresenting,
	FCustomDeltaChangelistState* InDeltaChangelistState)

	: CustomDeltaChangelistState(InDeltaChangelistState)
	, HistoryStart(0)
	, HistoryEnd(0)
	, CompareIndex(0)
	, StaticBuffer(InRepLayout->CreateShadowBuffer(InSource))

#if WITH_PUSH_MODEL
	, PushModelObjectHandle(UE_RepLayout_Private::ConditionallyAddPushModelObject(InRepresenting, InRepLayout))
#endif // WITH_PUSH_MODEL

{
#if (WITH_PUSH_MODEL)
	if (PushModelObjectHandle.IsValid())
	{
		FNetPrivatePushIdHelper::SetNetPushID(const_cast<UObject*>(InRepresenting), PushModelObjectHandle.ObjectId);
	}
#endif // (WITH_PUSH_MODEL)
}

FRepChangelistState::~FRepChangelistState()
{
#if WITH_PUSH_MODEL
	UE_RepLayout_Private::ConditionallyRemovePushModelObject(PushModelObjectHandle);
#endif // WITH_PUSH_MODEL

	// Explicitly reset members to improve resilience to double-destruction
	CustomDeltaChangelistState = nullptr;
	StaticBuffer.Empty();
}

#if WITH_PUSH_MODEL
bool FRepChangelistState::HasAnyDirtyProperties() const
{
	return UEPushModelPrivate::DoesHaveDirtyPropertiesOrRecentlyCollectedGarbage(PushModelObjectHandle);
}

bool FRepChangelistState::HasValidPushModelHandle() const
{
	return PushModelObjectHandle.IsValid();
}
#endif

void FRepChangelistState::CountBytes(FArchive& Ar) const
{
	GRANULAR_NETWORK_MEMORY_TRACKING_INIT(Ar, "FRepChangelistState::CountBytes");

	GRANULAR_NETWORK_MEMORY_TRACKING_TRACK("ChangeHistory",
		for (const FRepChangedHistory& HistoryItem : ChangeHistory)
		{
			HistoryItem.CountBytes(Ar);
		}
	);

	GRANULAR_NETWORK_MEMORY_TRACKING_TRACK("StaticBuffer", StaticBuffer.CountBytes(Ar));
	GRANULAR_NETWORK_MEMORY_TRACKING_TRACK("SharedSerialization", SharedSerialization.CountBytes(Ar));

	if (CustomDeltaChangelistState)
	{
		GRANULAR_NETWORK_MEMORY_TRACKING_TRACK("CustomDeltaChangelistState",
			Ar.CountBytes(sizeof(FCustomDeltaChangelistState), sizeof(FCustomDeltaChangelistState));
			CustomDeltaChangelistState->CountBytes(Ar);
		);
	}
}

FReplicationChangelistMgr::FReplicationChangelistMgr(
	const TSharedRef<const FRepLayout>& InRepLayout,
	const uint8* InSource,
	const UObject* InRepresenting,
	FCustomDeltaChangelistState* DeltaChangelistState)

	: LastReplicationFrame(0)
	, LastInitialReplicationFrame(0)
	, RepChangelistState(InRepLayout, InSource, InRepresenting, DeltaChangelistState)
{
}

FReplicationChangelistMgr::~FReplicationChangelistMgr()
{
}

void FReplicationChangelistMgr::CountBytes(FArchive& Ar) const
{
	GRANULAR_NETWORK_MEMORY_TRACKING_INIT(Ar, "FReplicationChangelistMgr::CountBytes");
	GRANULAR_NETWORK_MEMORY_TRACKING_TRACK("RepChangelistState", RepChangelistState.CountBytes(Ar));
}

FReceivingRepState::FReceivingRepState(FRepStateStaticBuffer&& InStaticBuffer) 
	: StaticBuffer(MoveTemp(InStaticBuffer))
{
}

FRepLayout::FRepLayout()
	: Flags(ERepLayoutFlags::None)
	, Owner(nullptr)
{}

FRepLayout::~FRepLayout()
{
}

ERepLayoutResult FRepLayout::UpdateChangelistMgr(
	FSendingRepState* RESTRICT RepState,
	FReplicationChangelistMgr& InChangelistMgr,
	const UObject* InObject,
	const uint32 ReplicationFrame,
	const FReplicationFlags& RepFlags,
	const bool bForceCompare) const
{
	ERepLayoutResult Result = ERepLayoutResult::Success;

	if (GShareInitialCompareState)
	{
		// See if we can re-use the work already done on a previous connection
		// Rules:
		// 1. We have replicated this actor at least once this frame
		// 2. This is not initial replication or we have done an initial replication this frame as well
		if (!bForceCompare && GShareShadowState && (InChangelistMgr.LastReplicationFrame == ReplicationFrame) && (!RepFlags.bNetInitial || (InChangelistMgr.LastInitialReplicationFrame == ReplicationFrame)))
		{
			// If this is initial replication, or we have never replicated on this connection, force a role compare
			if (RepFlags.bNetInitial || (RepState->LastCompareIndex == 0))
			{
				FReplicationFlags TempFlags = RepFlags;
				TempFlags.bRolesOnly = true;
				Result = CompareProperties(RepState, &InChangelistMgr.RepChangelistState, (const uint8*)InObject, TempFlags, bForceCompare);
			}

			INC_DWORD_STAT_BY(STAT_NetSkippedDynamicProps, 1);
			return Result;
		}
	}
	else
	{
		// See if we can re-use the work already done on a previous connection
		// Rules:
		//	1. We always compare once per frame (i.e. check LastReplicationFrame == ReplicationFrame)
		//	2. We check LastCompareIndex > 1 so we can do at least one pass per connection to compare all properties
		//		This is necessary due to how RemoteRole is manipulated per connection, so we need to give all connections a chance to see if it changed
		//	3. We ALWAYS compare on bNetInitial to make sure we have a fresh changelist of net initial properties in this case
		if (!bForceCompare && GShareShadowState && !RepFlags.bNetInitial && RepState->LastCompareIndex > 1 && InChangelistMgr.LastReplicationFrame == ReplicationFrame)
		{
			INC_DWORD_STAT_BY(STAT_NetSkippedDynamicProps, 1);
			return Result;
		}
	}

	Result = CompareProperties(RepState, &InChangelistMgr.RepChangelistState, (const uint8*)InObject, RepFlags, bForceCompare);

	// Currently, comparing properties should only result in Success, Empty, or FatalError.
	// So, don't bother checking for normal errors.
	if (LIKELY(ERepLayoutResult::FatalError != Result))
	{
		InChangelistMgr.LastReplicationFrame = ReplicationFrame;

		if (GShareInitialCompareState && RepFlags.bNetInitial)
		{
			InChangelistMgr.LastInitialReplicationFrame = ReplicationFrame;
		}
	}

	return Result;
}

struct FComparePropertiesSharedParams
{
	const bool bIsInitial;
	const bool bForceFail;
	const ERepLayoutFlags Flags;
	const TArray<FRepParentCmd>& Parents;
	const TArray<FRepLayoutCmd>& Cmds;
	FSendingRepState* const RepState;
	FRepChangelistState* const RepChangelistState;
	FRepChangedPropertyTracker* const RepChangedPropertyTracker;
	const TMap<FRepLayoutCmd*, TArray<FRepLayoutCmd>>& NetSerializeLayouts;
	UEPushModelPrivate::FPushModelPerNetDriverState* const PushModelState = nullptr;
	const TBitArray<>* const PushModelProperties = nullptr;
	const bool bValidateProperties = false;
	const bool bIsNetworkProfilerActive = false;
	const bool bChangedNetOwner = false;
	const bool bForceCustomPropsActive = false;
	const bool bForceCompareProperties = false;
#if (WITH_PUSH_VALIDATION_SUPPORT || USE_NETWORK_PROFILER)
	TBitArray<> PropertiesCompared;
	TBitArray<> PropertiesChanged;
#endif
};

struct FComparePropertiesStackParams
{
	const FConstRepObjectDataBuffer Data;
	FRepShadowDataBuffer ShadowData;
	TArray<uint16>& Changed;
	ERepLayoutResult& Result;
};

static uint16 CompareProperties_r(
	const FComparePropertiesSharedParams& SharedParams,
	FComparePropertiesStackParams& StackParams,
	const uint16 CmdStart,
	const uint16 CmdEnd,
	uint16 Handle);

static void CompareProperties_Array_r(
	const FComparePropertiesSharedParams& SharedParams,
	FComparePropertiesStackParams& StackParams,
	const uint16 CmdIndex,
	const uint16 Handle);

static bool CompareRoleProperty(
	const FComparePropertiesSharedParams& SharedParams,
	FComparePropertiesStackParams& StackParams,
	const uint16 RoleOrRemoteRoleIndex,
	TEnumAsByte<ENetRole>& SavedRoleOrRemoteRole)
{
	const FRepParentCmd& RoleOrRemoteRoleParent = SharedParams.Parents[RoleOrRemoteRoleIndex];
	const FRepLayoutCmd& RoleOrRemoteRoleCmd = SharedParams.Cmds[RoleOrRemoteRoleParent.CmdStart];
	const uint16 Handle = RoleOrRemoteRoleCmd.RelativeHandle;
	const TEnumAsByte<ENetRole> ActorRoleOrRemoteRole = *(const TEnumAsByte<ENetRole>*)(StackParams.Data + RoleOrRemoteRoleParent).Data;

	UE_LOGF(LogRepCompares, VeryVerbose, "CompareRoleProperty: bForceFail: %d, SavedRole: %ls ActorRole: %ls", SharedParams.bForceFail, *UEnum::GetValueAsString<ENetRole>(SavedRoleOrRemoteRole.GetValue()), *UEnum::GetValueAsString<ENetRole>(ActorRoleOrRemoteRole.GetValue()));

	if (SharedParams.bForceFail || SavedRoleOrRemoteRole != ActorRoleOrRemoteRole)
	{
		SavedRoleOrRemoteRole = ActorRoleOrRemoteRole;
		StackParams.Changed.Add(Handle);
		return true;
	}

	return false;
}

static void CompareRoleProperties(
	const FComparePropertiesSharedParams& SharedParams,
	FComparePropertiesStackParams& StackParams)
{
	if (SharedParams.RepState && EnumHasAnyFlags(SharedParams.Flags, ERepLayoutFlags::IsActor))
	{
		CompareRoleProperty(SharedParams, StackParams, (int32)AActor::ENetFields_Private::RemoteRole, SharedParams.RepState->SavedRemoteRole);
		CompareRoleProperty(SharedParams, StackParams, (int32)AActor::ENetFields_Private::Role, SharedParams.RepState->SavedRole);
	}
}

// Compare the specific FRepParentCmd.
// Returns true if the property (or any of its nested FRepLayoutCmds) has changed.
static bool CompareParentProperty(
	const int32 ParentIndex,
	const FComparePropertiesSharedParams& SharedParams,
	FComparePropertiesStackParams& StackParams)
{
	const FRepParentCmd& Parent = SharedParams.Parents[ParentIndex];
	const bool bIsLifetime = EnumHasAnyFlags(Parent.Flags, ERepParentFlags::IsLifetime);

	// Active state of a property applies to *all* connections.
	// If the property is inactive, we can skip comparing it because we know it won't be sent.
	// Further, this will keep the last active state of the property in the shadow buffer,
	// meaning the next time the property becomes active it will be sent to all connections.
	const bool bIsActive = SharedParams.bForceCustomPropsActive || !SharedParams.RepChangedPropertyTracker || SharedParams.RepChangedPropertyTracker->IsParentActive(ParentIndex);
	bool bShouldSkip = !bIsLifetime || !bIsActive;
	if (!bShouldSkip)
	{
		ELifetimeCondition Condition = Parent.Condition;
		if (Condition == COND_Dynamic)
		{
			Condition = SharedParams.RepChangedPropertyTracker ? SharedParams.RepChangedPropertyTracker->GetDynamicCondition(ParentIndex) : COND_Dynamic;
			bShouldSkip = Condition == COND_Never;
		}

		// Skip initial state if we're not replicating it.
		bShouldSkip |= !SharedParams.bIsInitial && Condition == COND_InitialOnly;
	}

	if (bShouldSkip)
	{
		return false;
	}

#if USE_NETWORK_PROFILER
	if (SharedParams.bIsNetworkProfilerActive)
	{
		const_cast<TBitArray<>&>(SharedParams.PropertiesCompared)[ParentIndex] = true;
	}
#endif

	const FRepLayoutCmd& Cmd = SharedParams.Cmds[Parent.CmdStart];

	if (EnumHasAnyFlags(SharedParams.Flags, ERepLayoutFlags::IsActor))
	{
		if (UNLIKELY(ParentIndex == (int32)AActor::ENetFields_Private::Role))
		{
			return CompareRoleProperty(SharedParams, StackParams, (int32)AActor::ENetFields_Private::Role, SharedParams.RepState->SavedRole);
		}
		if (UNLIKELY(ParentIndex == (int32)AActor::ENetFields_Private::RemoteRole))
		{
			return CompareRoleProperty(SharedParams, StackParams, (int32)AActor::ENetFields_Private::RemoteRole, SharedParams.RepState->SavedRemoteRole);
		}
	}
		
	const int32 NumChanges = StackParams.Changed.Num();

	// Note, Handle - 1 to account for CompareProperties_r incrementing handles.
	CompareProperties_r(SharedParams, StackParams, Parent.CmdStart, Parent.CmdEnd, Cmd.RelativeHandle - 1);

	return !!(StackParams.Changed.Num() - NumChanges);
}

namespace UE::Net::Private
{
	static bool CompareParentPropertyHelper(
		const int32 ParentIndex,
		const FComparePropertiesSharedParams& SharedParams,
		FComparePropertiesStackParams& StackParams)
	{
		const bool bDidPropertyChange = CompareParentProperty(ParentIndex, SharedParams, StackParams);

	#if USE_NETWORK_PROFILER
		if (SharedParams.bIsNetworkProfilerActive)
		{
			const_cast<TBitArray<>&>(SharedParams.PropertiesChanged)[ParentIndex] = bDidPropertyChange;
		}
	#endif // USING_NETWORK_PROFILER

		return bDidPropertyChange;
	}

#if WITH_PUSH_MODEL
	static bool IsPropertyDirty(
		const int32 ParentIndex,
		const bool bRecentlyCollectedGarbage,
		const FComparePropertiesSharedParams& SharedParams,
		FComparePropertiesStackParams& StackParams)
	{
		return SharedParams.bForceCompareProperties ||
			!(*SharedParams.PushModelProperties)[ParentIndex] || // non-push model properties are always considered dirty			
			SharedParams.PushModelState->IsPropertyDirty(ParentIndex) ||
			(bRecentlyCollectedGarbage &&
				EnumHasAnyFlags(SharedParams.Parents[ParentIndex].Flags, ERepParentFlags::HasObjectProperties | ERepParentFlags::IsNetSerialize));
	}
#endif // WITH_PUSH_MODEL	
}	

static void CompareParentProperties(
	const FComparePropertiesSharedParams& SharedParams,
	FComparePropertiesStackParams& StackParams)
{
	using namespace UE::Net::Private;
	check(StackParams.ShadowData);

#if WITH_PUSH_MODEL
	if (SharedParams.PushModelState != nullptr)
	{
		const bool bRecentlyCollectedGarbage = SharedParams.PushModelState->DidRecentlyCollectGarbage();

		// Typically, on an initial compare all properties will be dirty anyway.
		// However, if an Actor is awakened from Dormancy, it's SendingRepState will have been recreated, invalidating
		// its saved Role and RemoteRole (regardless of whether or not they were changed).
		// This really shouldn't cause any problems, but it will cause Validation to consistently fail (if GbPushModelValidateProperties
		// is enabled).
		// To get around this, and just to be sure our code paths are consistent, we forcibly mark Role and RemoteRole dirty here.
		// Also, if we changed Net Owner state, then our RemoteRole will have changed (due to FScopeRoleDowngrade) even though
		// the value hasn't changed on the Actor through normal gameplay code.
		// Instead of marking RemoteRole dirty in Scoped Role Downgrade (which would cause tons dirties), we detect that here.
		if (UNLIKELY(EnumHasAnyFlags(SharedParams.Flags, ERepLayoutFlags::IsActor) && (SharedParams.bIsInitial || SharedParams.bChangedNetOwner)))
		{
			SharedParams.PushModelState->MarkPropertyDirty((int32)AActor::ENetFields_Private::RemoteRole);
			SharedParams.PushModelState->MarkPropertyDirty((int32)AActor::ENetFields_Private::Role);
		}

		// If we're forcibly comparing all properties, then don't bother checking dirty state.
		if (UNLIKELY(SharedParams.bForceFail))
		{
			UE_LOGF(LogRepCompares, VeryVerbose, "CompareParentProperties: Force failed");

			for (int32 ParentIndex = 0; ParentIndex < SharedParams.Parents.Num(); ++ParentIndex)
			{
				CompareParentPropertyHelper(ParentIndex, SharedParams, StackParams);
			}
		}

#if (WITH_PUSH_VALIDATION_SUPPORT)		
		// If we're running validation, then we'll check everything regardless of push model state.
		else if (SharedParams.bValidateProperties)
		{
			UE_LOGF(LogRepCompares, VeryVerbose, "CompareParentProperties: Property validation");

			for (int32 ParentIndex = 0; ParentIndex < SharedParams.Parents.Num(); ++ParentIndex)
			{
				const ELifetimeCondition Condition = SharedParams.Parents[ParentIndex].Condition;
				const bool bRecompareInitialProperties = SharedParams.bIsInitial && (Condition == COND_InitialOnly || (Condition == COND_Dynamic && SharedParams.RepChangedPropertyTracker && SharedParams.RepChangedPropertyTracker->GetDynamicCondition(ParentIndex) == COND_InitialOnly));

				const bool bIsPropertyDirty = bRecompareInitialProperties || IsPropertyDirty(ParentIndex, bRecentlyCollectedGarbage, SharedParams, StackParams);
				
				const bool bDidPropertyChange = CompareParentPropertyHelper(ParentIndex, SharedParams, StackParams);

				ensureAlwaysMsgf(!bDidPropertyChange || bIsPropertyDirty, TEXT("Push Model Property changed value, but was not marked dirty! Property=%s"), *SharedParams.Parents[ParentIndex].Property->GetPathName());
			}	
		}
#endif // WITH_PUSH_VALIDATION_SUPPORT

		else if (UNLIKELY(SharedParams.bIsInitial && EnumHasAnyFlags(SharedParams.Flags, ERepLayoutFlags::HasInitialOnlyProperties | ERepLayoutFlags::HasDynamicConditionProperties)))
		{
			/*
				Most replication conditions don't actually effect whether or not we compare properties,
				because we share comparisons across connections, and typically only want to do one comparison
				per frame for all connections, when possible.
				
				This means that we will end up performing the comparison, ignoring replication conditions,
				even if we might not have to send the property to any connections on a given frame frame.
				
				COND_InitialOnly is special, though. In theory, COND_InitialOnly properties should only ever be
				compared a single time, the first time an object is ever replicated, so it's a waste to compare
				them every frame.

				However, in practice, there are a number of cases where an object can be replicated multiple
				times, and still have its bNetInitial flag set, because of how we handle multiple connections.
				E.G., it may be the first time an Object is replicating to a Specific Connection, and we would
				consider that an Initial Replication, even though the object may have replicated previously.

				If the property's value has changed, and the object was replicated again BEFORE a new
				Net Initial replication occurs, AND the object is a Push Model Property, then we will end up
				clearing the fact that the property was ever changed, and the new value will never be sent
				to any connections.

				To get around that, if we detect that this is an Initial Replication, and we have Initial Only
				properties, we will consider them dirty.
			*/

			UE_LOGF(LogRepCompares, VeryVerbose, "CompareParentProperties: Initial only test");

			for (int32 ParentIndex = 0; ParentIndex < SharedParams.Parents.Num(); ++ParentIndex)
			{
				const ELifetimeCondition Condition = SharedParams.Parents[ParentIndex].Condition;
				if (Condition == COND_InitialOnly || IsPropertyDirty(ParentIndex, bRecentlyCollectedGarbage, SharedParams, StackParams) ||
					(Condition == COND_Dynamic && SharedParams.RepChangedPropertyTracker && SharedParams.RepChangedPropertyTracker->GetDynamicCondition(ParentIndex) == COND_InitialOnly))
				{
					CompareParentPropertyHelper(ParentIndex, SharedParams, StackParams);
				}
			}
		}

		// If we have full push model property support, then we only need to check properties that are actually dirty.
		else if (EnumHasAnyFlags(SharedParams.Flags, ERepLayoutFlags::FullPushProperties) && !bRecentlyCollectedGarbage)
		{
			UE_LOGF(LogRepCompares, VeryVerbose, "CompareParentProperties: Full push properties: Has Dirty: %d", !!SharedParams.PushModelState->HasDirtyProperties());

			for (TConstSetBitIterator<> It = SharedParams.PushModelState->GetDirtyProperties(); It; ++It)
			{
				CompareParentPropertyHelper(It.GetIndex(), SharedParams, StackParams);
			}
		}
		else
		{
			UE_LOGF(LogRepCompares, VeryVerbose, "CompareParentProperties: Default");

			for (int32 ParentIndex = 0; ParentIndex < SharedParams.Parents.Num(); ++ParentIndex)
			{
				if (IsPropertyDirty(ParentIndex, bRecentlyCollectedGarbage, SharedParams, StackParams))
				{
					CompareParentPropertyHelper(ParentIndex, SharedParams, StackParams);
				}
			}
		}

		SharedParams.PushModelState->ResetDirtyStates();
		return;
	}
#endif // WITH_PUSH_MODEL

	for (int32 ParentIndex = 0; ParentIndex < SharedParams.Parents.Num(); ++ParentIndex)
	{
		CompareParentPropertyHelper(ParentIndex, SharedParams, StackParams);
	}
}


static uint16 CompareProperties_r(
	const FComparePropertiesSharedParams& SharedParams,
	FComparePropertiesStackParams& StackParams,
	const uint16 CmdStart,
	const uint16 CmdEnd,
	uint16 Handle)
{
	for (int32 CmdIndex = CmdStart; CmdIndex < CmdEnd; ++CmdIndex)
	{
		const FRepLayoutCmd& Cmd = SharedParams.Cmds[CmdIndex];

		check(Cmd.Type != ERepLayoutCmdType::Return);

		++Handle;

		const FConstRepObjectDataBuffer Data = StackParams.Data + Cmd;
		FRepShadowDataBuffer ShadowData = StackParams.ShadowData + Cmd;

		UE_LOGF(LogRepCompares, VeryVerbose, "CompareProperties_r: CmdIndex: %d CmdType: %ls Property: %ls", CmdIndex, LexToString(Cmd.Type), *GetNameSafe(Cmd.Property));

		if (Cmd.Type == ERepLayoutCmdType::DynamicArray)
		{
			FComparePropertiesStackParams NewStackParams{
				Data,
				ShadowData,
				StackParams.Changed,
				StackParams.Result
			};

			// Once we hit an array, start using a stack based approach
			CompareProperties_Array_r(SharedParams, NewStackParams, CmdIndex, Handle);
			CmdIndex = Cmd.EndCmd - 1;		// The -1 to handle the ++ in the for loop
			continue;
		}
		else if (SharedParams.bForceFail || !PropertiesAreIdentical(Cmd, ShadowData.Data, Data.Data, SharedParams.NetSerializeLayouts))
		{
			StoreProperty(Cmd, ShadowData.Data, Data.Data);
			StackParams.Changed.Add(Handle);
		}
	}

	return Handle;
}

static void CompareProperties_Array_r(
	const FComparePropertiesSharedParams& SharedParams,
	FComparePropertiesStackParams& StackParams,
	const uint16 CmdIndex,
	const uint16 Handle)
{
	const FRepLayoutCmd& Cmd = SharedParams.Cmds[CmdIndex];
	if (EnumHasAnyFlags(Cmd.Flags, ERepLayoutCmdFlags::IsEmptyArrayStruct))
	{
		return;
	}

	FScriptArray* ShadowArray = (FScriptArray*)StackParams.ShadowData.Data;
	FScriptArray* Array = (FScriptArray*)StackParams.Data.Data;

	const int32 ArrayNum = Array->Num();
	const int32 ShadowArrayNum = ShadowArray->Num();

	if (!UE_RepLayout_Private::ValidateArraySize(ArrayNum, Cmd.Property))
	{
		StackParams.Result = ERepLayoutResult::FatalError;
		return;
	}

	// Make the shadow state match the actual state at the time of compare
	FScriptArrayHelper StoredArrayHelper((FArrayProperty*)Cmd.Property, ShadowArray);
	StoredArrayHelper.Resize(ArrayNum);

	TArray<uint16> ChangedLocal;

	uint16 LocalHandle = 0;

	const FConstRepObjectDataBuffer ArrayData(Array->GetData());
	FRepShadowDataBuffer ShadowArrayData(ShadowArray->GetData());

	{
		const bool bOldForceFail = SharedParams.bForceFail;
		bool& bForceFail = const_cast<bool&>(SharedParams.bForceFail);
		TGuardValue<bool> ForceFailGuard(bForceFail, bForceFail);

		UE_LOGF(LogRepCompares, VeryVerbose, "CompareProperties_Array_r: ArrayNum: %d", ArrayNum);

		for (int32 i = 0; i < ArrayNum; i++)
		{
			const int32 ArrayElementOffset = i * Cmd.ElementSize;
			bForceFail = bOldForceFail || i >= ShadowArrayNum;

			FComparePropertiesStackParams NewStackParams{
				ArrayData + ArrayElementOffset,
				ShadowArrayData + ArrayElementOffset,
				ChangedLocal,
				StackParams.Result
			};

			LocalHandle = CompareProperties_r(SharedParams, NewStackParams, CmdIndex + 1, Cmd.EndCmd - 1, LocalHandle);
		}
	}

	if (ChangedLocal.Num())
	{
		const int32 NumChangedEntries = ChangedLocal.Num();
		
		if (!UE_RepLayout_Private::ValidateArraySize(NumChangedEntries, Cmd.Property))
		{
			StackParams.Result = ERepLayoutResult::FatalError;
			return;
		}

		StackParams.Changed.Add(Handle);
		StackParams.Changed.Add((uint16)NumChangedEntries);		// This is so we can jump over the array if we need to
		StackParams.Changed.Append(ChangedLocal);
		StackParams.Changed.Add(0);
	}
	else if (ArrayNum != ShadowArrayNum)
	{
		// If nothing below us changed, we either shrunk, or we grew and our inner was an array that didn't have any elements
		check(ArrayNum < ShadowArrayNum || SharedParams.Cmds[CmdIndex + 1].Type == ERepLayoutCmdType::DynamicArray);

		// Array got smaller, send the array handle to force array size change
		StackParams.Changed.Add(Handle);
		StackParams.Changed.Add(0);
		StackParams.Changed.Add(0);
	}
}

ERepLayoutResult FRepLayout::CompareProperties(
	FSendingRepState* RESTRICT RepState,
	FRepChangelistState* RESTRICT RepChangelistState,
	const FConstRepObjectDataBuffer Data,
	const FReplicationFlags& RepFlags,
	const bool bForceCompare) const
{
	CONDITIONAL_SCOPE_CYCLE_COUNTER(STAT_NetReplicateDynamicPropCompareTime, GUseDetailedScopeCounters);

	if (IsEmpty())
	{
		return ERepLayoutResult::Empty;
	}

	RepChangelistState->CompareIndex++;

	check((RepChangelistState->HistoryEnd - RepChangelistState->HistoryStart) < FRepChangelistState::MAX_CHANGE_HISTORY);
	const int32 HistoryIndex = RepChangelistState->HistoryEnd % FRepChangelistState::MAX_CHANGE_HISTORY;

	FRepChangedHistory& NewHistoryItem = RepChangelistState->ChangeHistory[HistoryIndex];

	UE_LOGF(LogRepCompares, VeryVerbose, "CompareProperties: Owner: %ls CompareIndex: %d HistoryIndex: %d", *GetFullNameSafe(Owner), RepChangelistState->CompareIndex, HistoryIndex);

	TArray<uint16>& Changed = NewHistoryItem.Changed;
	Changed.Empty(1);

#if WITH_PUSH_MODEL
	const TBitArray<>* const LocalPushModelProperties = &PushModelProperties;
#else
	const TBitArray<>* const LocalPushModelProperties = nullptr;
#endif

	ERepLayoutResult Result = ERepLayoutResult::Success;

	FComparePropertiesSharedParams SharedParams
	{
		.bIsInitial = !!RepFlags.bNetInitial,
		.bForceFail = !!RepFlags.bNetInitial && !!RepFlags.bForceInitialDirty,
		.Flags = Flags,
		.Parents = Parents,
		.Cmds = Cmds,
		.RepState = RepState,
		.RepChangelistState = RepChangelistState,
		.RepChangedPropertyTracker = (RepState ? RepState->RepChangedPropertyTracker.Get() : nullptr),
		.NetSerializeLayouts = NetSerializeLayouts,
		.PushModelState = UE_RepLayout_Private::GetPerNetDriverState(RepChangelistState),
		.PushModelProperties = LocalPushModelProperties,
		.bValidateProperties = GbPushModelValidateProperties,
		.bIsNetworkProfilerActive = UE_RepLayout_Private::IsNetworkProfilerComparisonTrackingEnabled(),
		.bChangedNetOwner = RepState && RepState->RepFlags.bNetOwner != RepFlags.bNetOwner,
		.bForceCustomPropsActive = !!RepFlags.bClientReplay,
		.bForceCompareProperties = bForceCompare
	};

	FComparePropertiesStackParams StackParams
	{
		.Data = Data,
		.ShadowData = RepChangelistState->StaticBuffer.GetData(),
		.Changed = Changed,
		.Result = Result
	};

	if (RepFlags.bRolesOnly)
	{
		// Don't track anything for the network profiler for this, since we want to use this frames *main* comparison
		// data for tracking.
		// This may throw off the number of total comparisons / replications for role properties, but this should
		// happen infrequently and the overall time and bandwidth spent on these properties is likely negligible
		// in the overall profile.
		CompareRoleProperties(SharedParams, StackParams);
	}
	else
	{
#if USE_NETWORK_PROFILER 
		const uint32 ReplicateParentPropertiesStartTime = SharedParams.bIsNetworkProfilerActive ? FPlatformTime::Cycles() : 0;
		if (SharedParams.bIsNetworkProfilerActive)
		{
			CSV_SCOPED_TIMING_STAT_EXCLUSIVE(NetworkProfiler);
			SharedParams.PropertiesChanged.Init(false, Parents.Num());
			SharedParams.PropertiesCompared.Init(false, Parents.Num());
		}

		struct FPropertyNameHelper
		{
			static FString ConvertParentCmdToPropertyName(const FRepParentCmd& Parent)
			{
				return Parent.CachedPropertyName.ToString();
			}
		};
#endif
	
		CompareParentProperties(SharedParams, StackParams);

		if (SharedParams.bIsNetworkProfilerActive)
		{
			NETWORK_PROFILER(GNetworkProfiler.TrackCompareProperties(Owner, FPlatformTime::Cycles() - ReplicateParentPropertiesStartTime, SharedParams.PropertiesCompared, SharedParams.PropertiesChanged, Parents, &FPropertyNameHelper::ConvertParentCmdToPropertyName););
		}
	}

	// If something went wrong, don't touch the changelist history.
	// NOTE: Currently, SA always throws "Warning: Expression 'ERepLayoutResult::Success != Result' is always false" here.
	//			It gets tripped up by the indirection / reference semantics of Stack Params and assumes that
	//			result just remains the same value as it was assigned above.
	//			Because of this, the error is disabled.
	if (UNLIKELY(ERepLayoutResult::Success != Result)) // -V547
	{
		return Result;
	}

	if (Changed.Num() == 0)
	{
		return ERepLayoutResult::Empty;
	}

	//
	// We produced a new change list, copy it to the history
	//

	// Null terminator
	Changed.Add(0);

	// Move end pointer
	RepChangelistState->HistoryEnd++;

	// New changes found so clear any existing shared serialization state
	RepChangelistState->SharedSerialization.Reset();

	// If we're full, merge the oldest up, so we always have room for a new entry
	if ((RepChangelistState->HistoryEnd - RepChangelistState->HistoryStart) == FRepChangelistState::MAX_CHANGE_HISTORY)
	{
		const int32 FirstHistoryIndex = RepChangelistState->HistoryStart % FRepChangelistState::MAX_CHANGE_HISTORY;

		RepChangelistState->HistoryStart++;

		const int32 SecondHistoryIndex = RepChangelistState->HistoryStart % FRepChangelistState::MAX_CHANGE_HISTORY;

		TArray<uint16>& FirstChangelistRef = RepChangelistState->ChangeHistory[FirstHistoryIndex].Changed;
		TArray<uint16> SecondChangelistCopy = MoveTemp(RepChangelistState->ChangeHistory[SecondHistoryIndex].Changed);

		MergeChangeList(Data, FirstChangelistRef, SecondChangelistCopy, RepChangelistState->ChangeHistory[SecondHistoryIndex].Changed);
	}

	return Result;
}

static FORCEINLINE void WritePropertyHandle(
	FNetBitWriter& Writer,
	uint16 Handle,
	bool bDoChecksum)
{
	const int NumStartingBits = Writer.GetNumBits();

	UE_NET_TRACE_SCOPE(PropertyHandle, Writer, GetTraceCollector(Writer), ENetTraceVerbosity::Trace);

	uint32 LocalHandle = Handle;
	Writer.SerializeIntPacked(LocalHandle);

	UE_LOGF(LogRepProperties, VeryVerbose, "WritePropertyHandle: Handle=%d", Handle);

#ifdef ENABLE_PROPERTY_CHECKSUMS
	if (bDoChecksum)
	{
		SerializeGenericChecksum(Writer);
	}
#endif

	NETWORK_PROFILER(GNetworkProfiler.TrackWritePropertyHandle(Writer.GetNumBits() - NumStartingBits, nullptr));
}

static void WritePropertyName(
	FNetBitWriter& Writer,
	const FName& PropertyName,
	bool bDoChecksum)
{
	const int NumStartingBits = Writer.GetNumBits();

	UE_NET_TRACE_SCOPE(PropertyName, Writer, GetTraceCollector(Writer), ENetTraceVerbosity::Trace);

	FName LocalPropertyName = PropertyName;
	Writer << LocalPropertyName;

	UE_LOGF(LogRepProperties, VeryVerbose, "WritePropertyName: Name=%ls", *PropertyName.ToString());

#ifdef ENABLE_PROPERTY_CHECKSUMS
	if (bDoChecksum)
	{
		SerializeGenericChecksum(Writer);
	}
#endif

	// TODO: Write a separate network profiler function for tracking that the name is being written instead
	// of the handle.
	NETWORK_PROFILER(GNetworkProfiler.TrackWritePropertyHandle(Writer.GetNumBits() - NumStartingBits, nullptr));
}

bool FRepLayout::ReplicateProperties(
	FSendingRepState* RESTRICT RepState,
	FRepChangelistState* RESTRICT RepChangelistState,
	const FConstRepObjectDataBuffer Data,
	UClass* ObjectClass,
	UActorChannel* OwningChannel,
	FNetBitWriter& Writer,
	const FReplicationFlags& RepFlags) const
{
	CONDITIONAL_SCOPE_CYCLE_COUNTER(STAT_NetReplicateDynamicPropTime, GUseDetailedScopeCounters);

	check(ObjectClass == Owner);

	// If we are an empty RepLayout, there's nothing to do.
	if (IsEmpty())
	{
		return false;
	}

	FRepChangedPropertyTracker*	ChangeTracker = RepState->RepChangedPropertyTracker.Get();

	const bool bRecordingCheckpoint = (OwningChannel->Connection->ResendAllDataState != EResendAllDataState::None);

	TArray<uint16> NewlyActiveChangelist;

	// Rebuild conditional state if needed
	if (RepState->RepFlags.Value != RepFlags.Value)
	{
		RebuildConditionalProperties(RepState, RepFlags);

		// Filter out any previously inactive changes from still inactive ones
		TArray<uint16> InactiveChangelist = MoveTemp(RepState->InactiveChangelist);
		TArray<uint16> NewInactiveChangeList;

		FilterChangeList(InactiveChangelist, RepState->InactiveParents, NewInactiveChangeList, NewlyActiveChangelist);

		// If we're recording a checkpoint, restore the inactive changelist
		if (bRecordingCheckpoint)
		{
			RepState->InactiveChangelist = MoveTemp(InactiveChangelist);
		}
		else
		{
			RepState->InactiveChangelist = MoveTemp(NewInactiveChangeList);
		}
	}

	if (OwningChannel->Connection->ResendAllDataState == EResendAllDataState::SinceOpen)
	{
		check(OwningChannel->Connection->IsInternalAck());

		if (RepState->LifetimeChangelist.Num() == 0)
		{
			// If this object was dormant, the lifetime list will be empty if the replicator was discarded, so rebuild it from the changelist state.
			for (int32 i = RepChangelistState->HistoryStart; i < RepChangelistState->HistoryEnd; ++i)
			{
				const int32 HistoryIndex = i % FRepChangelistState::MAX_CHANGE_HISTORY;

				FRepChangedHistory& HistoryItem = RepChangelistState->ChangeHistory[HistoryIndex];

				TArray<uint16> Temp = MoveTemp(RepState->LifetimeChangelist);
				MergeChangeList(Data, HistoryItem.Changed, Temp, RepState->LifetimeChangelist);
			}
		}

		// If we are resending data since open, we don't want to affect the current state of channel/replication, so just do the minimum and send the data, and return
		if (RepState->LifetimeChangelist.Num() > 0)
		{
			// Use a pruned version of the list, in case arrays changed size since the last time we replicated
			TArray<uint16> Pruned;
			PruneChangeList(Data, RepState->LifetimeChangelist, Pruned);
			RepState->LifetimeChangelist = MoveTemp(Pruned);

			// No need to merge in the newly active properties here, as the Lifetime Changelist should contain everything
			// inactive or otherwise.
			FilterChangeListToActive(RepState->LifetimeChangelist, RepState->InactiveParents, Pruned);
			if (Pruned.Num() > 0)
			{
				SendProperties_BackwardsCompatible(RepState, ChangeTracker, Data, OwningChannel->Connection, Writer, Pruned);
				return true;
			}
		}

		return false;
	}

	check(RepState->HistoryEnd >= RepState->HistoryStart);
	check((RepState->HistoryEnd - RepState->HistoryStart) < FSendingRepState::MAX_CHANGE_HISTORY);

	const bool bFlushPreOpenAckHistory = RepState->bOpenAckedCalled && RepState->PreOpenAckHistory.Num()> 0;

	const bool bCompareIndexSame = RepState->LastCompareIndex == RepChangelistState->CompareIndex;

	RepState->LastCompareIndex = RepChangelistState->CompareIndex;

	// We can early out if we know for sure there are no new changelists to send
	if (bCompareIndexSame || RepState->LastChangelistIndex == RepChangelistState->HistoryEnd)
	{
		if (RepState->NumNaks == 0 && !bFlushPreOpenAckHistory && NewlyActiveChangelist.Num() == 0)
		{
			// Nothing changed and there are no nak's, so just do normal housekeeping and remove acked history items
			UpdateChangelistHistory(RepState, ObjectClass, Data, OwningChannel->Connection, NULL);
			return false;
		}
	}

	// Clamp to the valid history range (and log if we end up sending entire history, this should only happen if we get really far behind)
	//	NOTE - The RepState->LastChangelistIndex != 0 should handle/ignore the JIP case
	if (RepState->LastChangelistIndex <= RepChangelistState->HistoryStart)
	{
		if (RepState->LastChangelistIndex != 0)
		{
			UE_LOGF(LogRep, Verbose, "FRepLayout::ReplicatePropertiesUsingChangelistState: Entire history sent for: %ls", *GetNameSafe(ObjectClass));
		}

		RepState->LastChangelistIndex = RepChangelistState->HistoryStart;
	}

	const int32 PossibleNewHistoryIndex = RepState->HistoryEnd % FSendingRepState::MAX_CHANGE_HISTORY;

	FRepChangedHistory& PossibleNewHistoryItem = RepState->ChangeHistory[PossibleNewHistoryIndex];

	TArray<uint16>& Changed = PossibleNewHistoryItem.Changed;

	// Make sure this history item is actually inactive
	check(Changed.Num() == 0);

	// Gather all change lists that are new since we last looked, and merge them all together into a single CL
	for (int32 i = RepState->LastChangelistIndex; i < RepChangelistState->HistoryEnd; ++i)
	{
		const int32 HistoryIndex = i % FRepChangelistState::MAX_CHANGE_HISTORY;

		FRepChangedHistory& HistoryItem = RepChangelistState->ChangeHistory[HistoryIndex];

		TArray<uint16> Temp = MoveTemp(Changed);
		MergeChangeList(Data, HistoryItem.Changed, Temp, Changed);
	}

	// Merge in newly active properties so they can be sent.
	if (NewlyActiveChangelist.Num() > 0)
	{
		TArray<uint16> Temp = MoveTemp(Changed);
		MergeChangeList(Data, NewlyActiveChangelist, Temp, Changed);
	}

	// We're all caught up now
	RepState->LastChangelistIndex = RepChangelistState->HistoryEnd;

	if (Changed.Num() > 0 || RepState->NumNaks > 0 || bFlushPreOpenAckHistory)
	{
		RepState->HistoryEnd++;

		UpdateChangelistHistory(RepState, ObjectClass, Data, OwningChannel->Connection, &Changed);

		// Merge in the PreOpenAckHistory (unreliable properties sent before the bunch was initially acked)
		if (bFlushPreOpenAckHistory)
		{
			for (int32 i = 0; i < RepState->PreOpenAckHistory.Num(); i++)
			{
				TArray<uint16> Temp = MoveTemp(Changed);
				MergeChangeList(Data, RepState->PreOpenAckHistory[i].Changed, Temp, Changed);
			}
			RepState->PreOpenAckHistory.Empty();
		}
	}
	else
	{
		// Nothing changed and there are no nak's, so just do normal housekeeping and remove acked history items
		UpdateChangelistHistory(RepState, ObjectClass, Data, OwningChannel->Connection, nullptr);
		return false;
	}

	// At this point we should have a non empty change list
	check(Changed.Num() > 0);

	// do not build shared state for InternalAck (demo) connections
	if (!OwningChannel->Connection->IsInternalAck() && (GNetSharedSerializedData != 0))
	{
		// if no shared serialization info exists, build it
		if (!RepChangelistState->SharedSerialization.IsValid())
		{
			UE::Net::FNetTokenStore* NetTokenStore = OwningChannel->Connection->GetDriver()->GetNetTokenStore();
			BuildSharedSerialization(Data, Changed, true, RepChangelistState->SharedSerialization, NetTokenStore);
		}
	}

	const int32 NumBits = Writer.GetNumBits();

	// Filter out the final changelist into Active and Inactive.
	TArray<uint16> UnfilteredChanged = MoveTemp(Changed);
	TArray<uint16> NewlyInactiveChangelist;
	FilterChangeList(UnfilteredChanged, RepState->InactiveParents, NewlyInactiveChangelist, Changed);

	// If we have any properties that are no longer active, make sure we track them.
	if (!bRecordingCheckpoint && (NewlyInactiveChangelist.Num() > 1))
	{
		TArray<uint16> Temp = MoveTemp(RepState->InactiveChangelist);
		MergeChangeList(Data, NewlyInactiveChangelist, Temp, RepState->InactiveChangelist);
	}

	// Send the final merged change list
	if (OwningChannel->Connection->IsInternalAck() && !RepFlags.bSerializePropertyNames)
	{
		// Remember all properties that have changed since this channel was first opened in case we need it (for bResendAllDataSinceOpen)
		// We use UnfilteredChanged so LifetimeChangelist contains all properties, regardless of Active state.
		TArray<uint16> Temp = MoveTemp(RepState->LifetimeChangelist);
		MergeChangeList(Data, UnfilteredChanged, Temp, RepState->LifetimeChangelist);

		if (Changed.Num() > 0)
		{
			SendProperties_BackwardsCompatible(RepState, ChangeTracker, Data, OwningChannel->Connection, Writer, Changed);
		}
	}
	else if (Changed.Num() > 0)
	{
		SendProperties(RepState, ChangeTracker, Data, ObjectClass, Writer, Changed, RepChangelistState->SharedSerialization, RepFlags.bSerializePropertyNames ? ESerializePropertyType::Name : ESerializePropertyType::Handle);

		if (UE::Net::FNetTokenExportContext* NetTokenExportContext = RepChangelistState->SharedSerialization.IsValid() ? Writer.NetTokenExportContext.Get() : nullptr)
		{
			// For now just append all potential exports, we can be smarter about it if necessary
			NetTokenExportContext->AppendNetTokensPendingExport(RepChangelistState->SharedSerialization.NetTokensPendingExport);
		}
	}

	// See if something actually sent (this may be false due to conditional checks inside the send properties function
	const bool bSomethingSent = NumBits != Writer.GetNumBits();

	if (!bSomethingSent)
	{
		// We need to revert the change list in the history if nothing really sent (can happen due to condition checks)
		Changed.Empty();
		RepState->HistoryEnd--;
	}

	return bSomethingSent;
}

void FRepLayout::UpdateChangelistHistory(
	FSendingRepState* RepState,
	UClass* ObjectClass,
	const FConstRepObjectDataBuffer Data,
	UNetConnection* Connection,
	TArray<uint16>* OutMerged) const
{
	check(RepState->HistoryEnd >= RepState->HistoryStart);

	const int32 HistoryCount = RepState->HistoryEnd - RepState->HistoryStart;
	const bool bDumpHistory = HistoryCount == FSendingRepState::MAX_CHANGE_HISTORY;
	const int32 AckPacketId = Connection->OutAckPacketId;

	// If our buffer is currently full, forcibly send the entire history
	if (bDumpHistory)
	{
		UE_LOGF(LogRep, Verbose, "FRepLayout::UpdateChangelistHistory: History overflow, forcing history dump %ls, %ls", *ObjectClass->GetName(), *Connection->Describe());
	}

	const bool bDeltaCheckpoint = (Connection->ResendAllDataState == EResendAllDataState::SinceCheckpoint);

	if (bDeltaCheckpoint)
	{
		for (int32 i = RepState->HistoryStart; i < RepState->HistoryEnd - 1; i++)
		{
			const int32 HistoryIndex = i % FSendingRepState::MAX_CHANGE_HISTORY;

			FRepChangedHistory& HistoryItem = RepState->ChangeHistory[HistoryIndex];

			// All active history items should contain a change list
			check(HistoryItem.Changed.Num() > 0);

			HistoryItem.Reset();
			RepState->HistoryStart++;
		}
	}
	else
	{
		for (int32 i = RepState->HistoryStart; i < RepState->HistoryEnd; i++)
		{
			const int32 HistoryIndex = i % FSendingRepState::MAX_CHANGE_HISTORY;

			FRepChangedHistory& HistoryItem = RepState->ChangeHistory[HistoryIndex];

			if (HistoryItem.OutPacketIdRange.First == INDEX_NONE)
			{
				// Hasn't been initialized in PostReplicate yet
				// No need to go further, otherwise we'll overwrite entries incorrectly.
				break;
			}

			// All active history items should contain a change list
			check(HistoryItem.Changed.Num() > 0);

			if (AckPacketId >= HistoryItem.OutPacketIdRange.Last || HistoryItem.Resend || bDumpHistory)
			{
				if (HistoryItem.Resend || bDumpHistory)
				{
					// Merge in nak'd change lists
					check(OutMerged != NULL);
					TArray<uint16> Temp = MoveTemp(*OutMerged);
					MergeChangeList(Data, HistoryItem.Changed, Temp, *OutMerged);

#ifdef SANITY_CHECK_MERGES
					SanityCheckChangeList(Data, *OutMerged);
#endif

					if (HistoryItem.Resend)
					{
						RepState->NumNaks--;
					}
				}

				HistoryItem.Reset();
				RepState->HistoryStart++;
			}
		}
	}

	// Remove any tiling in the history markers to keep them from wrapping over time
	const int32 NewHistoryCount	= RepState->HistoryEnd - RepState->HistoryStart;

	check(NewHistoryCount < FSendingRepState::MAX_CHANGE_HISTORY);

	RepState->HistoryStart = RepState->HistoryStart % FSendingRepState::MAX_CHANGE_HISTORY;
	RepState->HistoryEnd = RepState->HistoryStart + NewHistoryCount;

	// Make sure we processed all the naks properly
	check(RepState->NumNaks == 0);
}

void FRepLayout::SerializeObjectReplicatedProperties(UObject* Object, FBitArchive & Ar) const
{
	static FRepSerializationSharedInfo Empty;

	for (int32 i = 0; i < Parents.Num(); i++)
	{
		FStructProperty* StructProperty = CastField<FStructProperty>(Parents[i].Property);
		FObjectProperty* ObjectProperty = CastField<FObjectProperty>(Parents[i].Property);

		// We're only able to easily serialize non-object/struct properties, so just do those.
		if (ObjectProperty == nullptr && StructProperty == nullptr)
		{
			bool bHasUnmapped = false;
			SerializeProperties_r(Ar, NULL, Parents[i].CmdStart, Parents[i].CmdEnd, (uint8*)Object, bHasUnmapped, 0, 0, Empty, nullptr, nullptr);
		}
	}
}

bool FRepHandleIterator::NextHandle()
{
	CmdIndex = INDEX_NONE;

	Handle = ChangelistIterator.Changed[ChangelistIterator.ChangedIndex];

	if (Handle == 0)
	{
		return false;		// Done
	}

	ChangelistIterator.ChangedIndex++;

	if (!ensureMsgf(ChangelistIterator.Changed.IsValidIndex(ChangelistIterator.ChangedIndex),
			TEXT("Attempted to access invalid iterator index: Handle=%d, ChangedIndex=%d, ChangedNum=%d, Owner=%s, LastSuccessfulCmd=%s"),
			Handle, ChangelistIterator.ChangedIndex, ChangelistIterator.Changed.Num(),
			*GetPathNameSafe(Owner),
			*((Cmds.IsValidIndex(LastSuccessfulCmdIndex) && Cmds[LastSuccessfulCmdIndex].Property) ? Cmds[LastSuccessfulCmdIndex].Property->GetPathName() : FString::FromInt(LastSuccessfulCmdIndex))))
	{
		return false;
	}

	const int32 HandleMinusOne = Handle - 1;

	ArrayIndex = (ArrayElementSize> 0 && NumHandlesPerElement> 0) ? HandleMinusOne / NumHandlesPerElement : 0;

	if (ArrayIndex >= MaxArrayIndex)
	{
		return false;
	}

	ArrayOffset	= ArrayIndex * ArrayElementSize;

	const int32 RelativeHandle = HandleMinusOne - ArrayIndex * NumHandlesPerElement;

	if (!ensureMsgf(HandleToCmdIndex.IsValidIndex(RelativeHandle),
			TEXT("Attempted to access invalid RelativeHandle Index: Handle=%d, RelativeHandle=%d, NumHandlesPerElement=%d, ArrayIndex=%d, ArrayElementSize=%d, Owner=%s, LastSuccessfulCmd=%s"),
			Handle, RelativeHandle, NumHandlesPerElement, ArrayIndex, ArrayElementSize,
			*GetPathNameSafe(Owner),
			*((Cmds.IsValidIndex(LastSuccessfulCmdIndex) && Cmds[LastSuccessfulCmdIndex].Property) ? Cmds[LastSuccessfulCmdIndex].Property->GetPathName() : FString::FromInt(LastSuccessfulCmdIndex))))
	{
		return false;
	}

	CmdIndex = HandleToCmdIndex[RelativeHandle].CmdIndex;

	if (!ensureMsgf(MinCmdIndex <= CmdIndex && CmdIndex < MaxCmdIndex,
			TEXT("Attempted to access Command Index outside of iterator range: Handle=%d, RelativeHandle=%d, CmdIndex=%d, MinCmdIdx=%d, MaxCmdIdx=%d, ArrayIndex=%d, Owner=%s, LastSuccessfulCmd=%s"),
			Handle, RelativeHandle, CmdIndex, MinCmdIndex, MaxCmdIndex, ArrayIndex,
			*GetPathNameSafe(Owner),
			*((Cmds.IsValidIndex(LastSuccessfulCmdIndex) && Cmds[LastSuccessfulCmdIndex].Property) ? Cmds[LastSuccessfulCmdIndex].Property->GetPathName() : FString::FromInt(LastSuccessfulCmdIndex))))
	{
		return false;
	}

	const FRepLayoutCmd& Cmd = Cmds[CmdIndex];

	if (!ensureMsgf(Cmd.RelativeHandle - 1 == RelativeHandle,
			TEXT("Command Relative Handle does not match found Relative Handle: Handle=%d, RelativeHandle=%d, CmdIdx=%d, CmdRelativeHandle=%d, ArrayIndex=%d, Owner=%s, LastSuccessfulCmd=%s"),
			Handle, RelativeHandle, CmdIndex, Cmd.RelativeHandle, ArrayIndex,
			*GetPathNameSafe(Owner),
			*((Cmds.IsValidIndex(LastSuccessfulCmdIndex) && Cmds[LastSuccessfulCmdIndex].Property) ? Cmds[LastSuccessfulCmdIndex].Property->GetPathName() : FString::FromInt(LastSuccessfulCmdIndex))))
	{
		return false;
	}

	if (!ensureMsgf(Cmd.Type != ERepLayoutCmdType::Return,
			TEXT("Hit unexpected return handle: Handle=%d, RelativeHandle=%d, CmdIdx=%d, ArrayIndex=%d, Owner=%s, LastSuccessfulCmd=%s"),
			Handle, RelativeHandle, CmdIndex, ArrayIndex,
			*GetPathNameSafe(Owner),
			*((Cmds.IsValidIndex(LastSuccessfulCmdIndex) && Cmds[LastSuccessfulCmdIndex].Property) ? Cmds[LastSuccessfulCmdIndex].Property->GetPathName() : FString::FromInt(LastSuccessfulCmdIndex))))
	{
		return false;
	}

	LastSuccessfulCmdIndex = CmdIndex;

	return true;
}

bool FRepHandleIterator::JumpOverArray()
{
	const int32 ArrayChangedCount = ChangelistIterator.Changed[ChangelistIterator.ChangedIndex++];
	ChangelistIterator.ChangedIndex += ArrayChangedCount;

	if (!ensure(ChangelistIterator.Changed[ChangelistIterator.ChangedIndex] == 0))
	{
		return false;
	}

	ChangelistIterator.ChangedIndex++;

	return true;
}

int32 FRepHandleIterator::PeekNextHandle() const
{
	return ChangelistIterator.Changed[ChangelistIterator.ChangedIndex];
}

class FScopedIteratorArrayTracker
{
public:
	FScopedIteratorArrayTracker(FRepHandleIterator* InCmdIndexIterator)
	{
		CmdIndexIterator = InCmdIndexIterator;

		if (CmdIndexIterator)
		{
			ArrayChangedCount = CmdIndexIterator->ChangelistIterator.Changed[CmdIndexIterator->ChangelistIterator.ChangedIndex++];
			OldChangedIndex = CmdIndexIterator->ChangelistIterator.ChangedIndex;
		}
	}

	~FScopedIteratorArrayTracker()
	{
		if (CmdIndexIterator)
		{
			check(CmdIndexIterator->ChangelistIterator.ChangedIndex - OldChangedIndex <= ArrayChangedCount);
			CmdIndexIterator->ChangelistIterator.ChangedIndex = OldChangedIndex + ArrayChangedCount;
			check(CmdIndexIterator->PeekNextHandle() == 0);
			CmdIndexIterator->ChangelistIterator.ChangedIndex++;
		}
	}

	FRepHandleIterator* CmdIndexIterator;
	int32 ArrayChangedCount;
	int32 OldChangedIndex;
};

void FRepLayout::MergeChangeList_r(
	FRepHandleIterator& RepHandleIterator1,
	FRepHandleIterator& RepHandleIterator2,
	const FConstRepObjectDataBuffer SourceData,
	TArray<uint16>& OutChanged) const
{
	while (true)
	{
		const int32 NextHandle1 = RepHandleIterator1.PeekNextHandle();
		const int32 NextHandle2 = RepHandleIterator2.PeekNextHandle();

		if (NextHandle1 == 0 && NextHandle2 == 0)
		{
			// Done
			break;
		}

		if (NextHandle2 == 0)
		{
			PruneChangeList_r(RepHandleIterator1, SourceData, OutChanged);
			return;
		}
		else if (NextHandle1 == 0)
		{
			PruneChangeList_r(RepHandleIterator2, SourceData, OutChanged);
			return;
		}

		FRepHandleIterator* ActiveIterator1 = nullptr;
		FRepHandleIterator* ActiveIterator2 = nullptr;

		int32 CmdIndex = INDEX_NONE;
		int32 ArrayOffset = INDEX_NONE;

		if (NextHandle1 < NextHandle2)
		{
			if (!RepHandleIterator1.NextHandle())
			{
				// Array overflow
				break;
			}

			OutChanged.Add(NextHandle1);

			CmdIndex = RepHandleIterator1.CmdIndex;
			ArrayOffset = RepHandleIterator1.ArrayOffset;

			ActiveIterator1 = &RepHandleIterator1;
		}
		else if (NextHandle2 < NextHandle1)
		{
			if (!RepHandleIterator2.NextHandle())
			{
				// Array overflow
				break;
			}

			OutChanged.Add(NextHandle2);

			CmdIndex = RepHandleIterator2.CmdIndex;
			ArrayOffset = RepHandleIterator2.ArrayOffset;

			ActiveIterator2 = &RepHandleIterator2;
		}
		else
		{
			check(NextHandle1 == NextHandle2);

			if (!RepHandleIterator1.NextHandle())
			{
				// Array overflow
				break;
			}

			if (!ensure(RepHandleIterator2.NextHandle()))
			{
				// Array overflow
				break;
			}

			check(RepHandleIterator1.CmdIndex == RepHandleIterator2.CmdIndex);

			OutChanged.Add(NextHandle1);

			CmdIndex = RepHandleIterator1.CmdIndex;
			ArrayOffset = RepHandleIterator1.ArrayOffset;

			ActiveIterator1 = &RepHandleIterator1;
			ActiveIterator2 = &RepHandleIterator2;
		}

		const FRepLayoutCmd& Cmd = Cmds[CmdIndex];

		if (Cmd.Type == ERepLayoutCmdType::DynamicArray)
		{
			const FConstRepObjectDataBuffer Data = (SourceData + Cmd) + ArrayOffset;
			const FScriptArray* Array = (FScriptArray *)Data.Data;
			const FConstRepObjectDataBuffer ArrayData(Array->GetData());

			FScopedIteratorArrayTracker ArrayTracker1(ActiveIterator1);
			FScopedIteratorArrayTracker ArrayTracker2(ActiveIterator2);

			const int32 OriginalChangedNum	= OutChanged.AddUninitialized();

			TArray<FHandleToCmdIndex>& ArrayHandleToCmdIndex = ActiveIterator1 ? *ActiveIterator1->HandleToCmdIndex[Cmd.RelativeHandle - 1].HandleToCmdIndex : *ActiveIterator2->HandleToCmdIndex[Cmd.RelativeHandle - 1].HandleToCmdIndex; //-V595

			if (!ActiveIterator1)
			{
				FRepHandleIterator ArrayIterator2(ActiveIterator2->Owner, ActiveIterator2->ChangelistIterator, Cmds, ArrayHandleToCmdIndex, Cmd.ElementSize, Array->Num(), CmdIndex + 1, Cmd.EndCmd - 1);
				PruneChangeList_r(ArrayIterator2, ArrayData, OutChanged);
			}
			else if (!ActiveIterator2)
			{
				FRepHandleIterator ArrayIterator1(ActiveIterator1->Owner, ActiveIterator1->ChangelistIterator, Cmds, ArrayHandleToCmdIndex, Cmd.ElementSize, Array->Num(), CmdIndex + 1, Cmd.EndCmd - 1);
				PruneChangeList_r(ArrayIterator1, ArrayData, OutChanged);
			}
			else
			{
				FRepHandleIterator ArrayIterator1(ActiveIterator1->Owner, ActiveIterator1->ChangelistIterator, Cmds, ArrayHandleToCmdIndex, Cmd.ElementSize, Array->Num(), CmdIndex + 1, Cmd.EndCmd - 1);
				FRepHandleIterator ArrayIterator2(ActiveIterator2->Owner, ActiveIterator2->ChangelistIterator, Cmds, ArrayHandleToCmdIndex, Cmd.ElementSize, Array->Num(), CmdIndex + 1, Cmd.EndCmd - 1);

				MergeChangeList_r(ArrayIterator1, ArrayIterator2, ArrayData, OutChanged);
			}

			// Patch in the jump offset
			OutChanged[OriginalChangedNum] = OutChanged.Num() - (OriginalChangedNum + 1);

			// Add the array terminator
			OutChanged.Add(0);
		}
	}
}

void FRepLayout::PruneChangeList_r(
	FRepHandleIterator& RepHandleIterator,
	const FConstRepObjectDataBuffer SourceData,
	TArray<uint16>& OutChanged) const
{
	while (RepHandleIterator.NextHandle())
	{
		OutChanged.Add(RepHandleIterator.Handle);

		const int32 CmdIndex = RepHandleIterator.CmdIndex;
		const int32 ArrayOffset = RepHandleIterator.ArrayOffset;

		const FRepLayoutCmd& Cmd = Cmds[CmdIndex];

		if (Cmd.Type == ERepLayoutCmdType::DynamicArray)
		{
			const FConstRepObjectDataBuffer Data = (SourceData + Cmd) + ArrayOffset;
			const FScriptArray* Array = (FScriptArray *)Data.Data;
			const FConstRepObjectDataBuffer ArrayData(Array->GetData());

			FScopedIteratorArrayTracker ArrayTracker(&RepHandleIterator);

			const int32 OriginalChangedNum = OutChanged.AddUninitialized();

			TArray<FHandleToCmdIndex>& ArrayHandleToCmdIndex = *RepHandleIterator.HandleToCmdIndex[Cmd.RelativeHandle - 1].HandleToCmdIndex;

			FRepHandleIterator ArrayIterator(RepHandleIterator.Owner, RepHandleIterator.ChangelistIterator, Cmds, ArrayHandleToCmdIndex, Cmd.ElementSize, Array->Num(), CmdIndex + 1, Cmd.EndCmd - 1);
			PruneChangeList_r(ArrayIterator, ArrayData, OutChanged);

			// Patch in the jump offset
			OutChanged[OriginalChangedNum] = OutChanged.Num() - (OriginalChangedNum + 1);

			// Add the array terminator
			OutChanged.Add(0);
		}
	}
}

void FRepLayout::FilterChangeList(
	const TArray<uint16>& Changelist,
	const TBitArray<>& InactiveParents,
	TArray<uint16>& OutInactiveProperties,
	TArray<uint16>& OutActiveProperties) const
{
	FChangelistIterator ChangelistIterator(Changelist, 0);
	FRepHandleIterator HandleIterator(Owner, ChangelistIterator, Cmds, BaseHandleToCmdIndex, 0, 1, 0, Cmds.Num() - 1);

	OutInactiveProperties.Empty(1);
	OutActiveProperties.Empty(1);

	while (HandleIterator.NextHandle())
	{
		const FRepLayoutCmd& Cmd = Cmds[HandleIterator.CmdIndex];

		TArray<uint16>& Properties = InactiveParents[Cmd.ParentIndex] ? OutInactiveProperties : OutActiveProperties;
			
		Properties.Add(HandleIterator.Handle);

		if (Cmd.Type == ERepLayoutCmdType::DynamicArray)
		{
			// No need to recursively filter the change list, as handles are only enabled/disabled at the parent level
			int32 HandleCount = ChangelistIterator.Changed[ChangelistIterator.ChangedIndex];
			Properties.Add(HandleCount);
					
			for (int32 I = 0; I < HandleCount; ++I)
			{
				Properties.Add(ChangelistIterator.Changed[ChangelistIterator.ChangedIndex + 1 + I]);
			}

			Properties.Add(0);

			HandleIterator.JumpOverArray();
		}
	}

	OutInactiveProperties.Add(0);
	OutActiveProperties.Add(0);
}

void FRepLayout::FilterChangeListToActive(
	const TArray<uint16>& Changelist,
	const TBitArray<>& InactiveParents,
	TArray<uint16>& OutProperties) const
{
	FChangelistIterator ChangelistIterator(Changelist, 0);
	FRepHandleIterator HandleIterator(Owner, ChangelistIterator, Cmds, BaseHandleToCmdIndex, 0, 1, 0, Cmds.Num() - 1);

	OutProperties.Empty(1);

	while (HandleIterator.NextHandle())
	{
		const FRepLayoutCmd& Cmd = Cmds[HandleIterator.CmdIndex];
		if (!InactiveParents[Cmd.ParentIndex])
		{
			OutProperties.Add(HandleIterator.Handle);

			if (Cmd.Type == ERepLayoutCmdType::DynamicArray)
			{
				// No need to recursively filter the change list, as handles are only enabled/disabled at the parent level
				int32 HandleCount = ChangelistIterator.Changed[ChangelistIterator.ChangedIndex];
				OutProperties.Add(HandleCount);

				for (int32 I = 0; I < HandleCount; ++I)
				{
					OutProperties.Add(ChangelistIterator.Changed[ChangelistIterator.ChangedIndex + 1 + I]);
				}

				OutProperties.Add(0);

				HandleIterator.JumpOverArray();
			}
		}
		else if (Cmd.Type == ERepLayoutCmdType::DynamicArray)
		{
			HandleIterator.JumpOverArray();
		}
	}

	OutProperties.Add(0);
}

void FRepSerializationSharedInfo::CountBytes(FArchive& Ar) const
{
	GRANULAR_NETWORK_MEMORY_TRACKING_INIT(Ar, "FRepSerializationSharedInfo::CountBytes");

	GRANULAR_NETWORK_MEMORY_TRACKING_TRACK("SharedPropertyInfo", SharedPropertyInfo.CountBytes(Ar));

	GRANULAR_NETWORK_MEMORY_TRACKING_TRACK("SerializedProperties",
		if (FNetBitWriter const* const LocalSerializedProperties = SerializedProperties.Get())
		{
			LocalSerializedProperties->CountMemory(Ar);
		}
	);
}

const FRepSerializedPropertyInfo* FRepSerializationSharedInfo::WriteSharedProperty(
	const FRepLayoutCmd& Cmd,
	const FRepSharedPropertyKey& PropertyKey,
	const int32 CmdIndex,
	const uint16 Handle,
	const FConstRepObjectDataBuffer Data,
	const bool bWriteHandle,
	const bool bDoChecksum)
{
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	check(!SharedPropertyInfo.ContainsByPredicate([PropertyKey](const FRepSerializedPropertyInfo& Info)
	{ 
		return (Info.PropertyKey == PropertyKey);
	}));
#endif

	check(SerializedProperties.IsValid());

	FRepSerializedPropertyInfo& SharedPropInfo = SharedPropertyInfo.Emplace_GetRef();

	SharedPropInfo.PropertyKey = PropertyKey;
	SharedPropInfo.BitOffset = SerializedProperties->GetNumBits();

	UE_LOGF(LogRepProperties, VeryVerbose, "WriteSharedProperty: Handle=%d, Key=%ls", Handle, *PropertyKey.ToDebugString());

	if (bWriteHandle)
	{
		WritePropertyHandle(*SerializedProperties, Handle, bDoChecksum);
	}

	SharedPropInfo.PropBitOffset = SerializedProperties->GetNumBits();

	// This property changed, so send it
	Cmd.Property->NetSerializeItem(*SerializedProperties, nullptr, const_cast<uint8*>(Data.Data));

	const int64 NumPropEndBits = SerializedProperties->GetNumBits();

	SharedPropInfo.PropBitLength = NumPropEndBits - SharedPropInfo.PropBitOffset;

#ifdef ENABLE_PROPERTY_CHECKSUMS
	if (bDoChecksum)
	{
		SerializeReadWritePropertyChecksum(Cmd, CmdIndex, Data, *SerializedProperties);
	}
#endif

	SharedPropInfo.BitLength = SerializedProperties->GetNumBits() - SharedPropInfo.BitOffset;

	return &SharedPropInfo;
}

void FRepLayout::SendProperties_r(
	FSendingRepState* RESTRICT RepState,
	FNetBitWriter& Writer,
	const bool bDoChecksum,
	FRepHandleIterator& HandleIterator,
	const FConstRepObjectDataBuffer SourceData,
	const int32 ArrayDepth,
	const FRepSerializationSharedInfo* const RESTRICT SharedInfo,
	const ESerializePropertyType SerializePropertyType) const
{
	const bool bDoSharedSerialization = SharedInfo && !!GNetSharedSerializedData;

	while (HandleIterator.NextHandle())
	{
		const FRepLayoutCmd& Cmd = Cmds[HandleIterator.CmdIndex];
		const FRepParentCmd& ParentCmd = Parents[Cmd.ParentIndex];

		UE_LOGF(LogRepProperties, VeryVerbose, "SendProperties_r: Parent=%d, Cmd=%d, ArrayIndex=%d", Cmd.ParentIndex, HandleIterator.CmdIndex, HandleIterator.ArrayIndex);
		
		FConstRepObjectDataBuffer Data = (SourceData + Cmd) + HandleIterator.ArrayOffset;

		if (Cmd.Type == ERepLayoutCmdType::DynamicArray)
		{
			if (SerializePropertyType == ESerializePropertyType::Handle)
			{
				WritePropertyHandle(Writer, HandleIterator.Handle, bDoChecksum);
			}
			else if (SerializePropertyType == ESerializePropertyType::Name)
			{
				WritePropertyName(Writer, Cmd.Property->GetFName(), bDoChecksum);
			}

			UE_NET_TRACE_DYNAMIC_NAME_SCOPE(Cmd.Property->GetFName(), Writer, GetTraceCollector(Writer), ENetTraceVerbosity::Trace);

			const FScriptArray* Array = (FScriptArray *)Data.Data;
			const FConstRepObjectDataBuffer ArrayData(Array->GetData());

			// Write array num
			uint16 ArrayNum = Array->Num();
			Writer << ArrayNum;

			UE_LOGF(LogRepProperties, VeryVerbose, "SendProperties_r: ArrayNum=%d", ArrayNum);

			// Read the jump offset
			// We won't need to actually jump over anything because we expect the change list to be pruned once we get here
			// But we can use it to verify we read the correct amount.
			const int32 ArrayChangedCount = HandleIterator.ChangelistIterator.Changed[HandleIterator.ChangelistIterator.ChangedIndex++];

			const int32 OldChangedIndex = HandleIterator.ChangelistIterator.ChangedIndex;

			TArray<FHandleToCmdIndex>& ArrayHandleToCmdIndex = *HandleIterator.HandleToCmdIndex[Cmd.RelativeHandle - 1].HandleToCmdIndex;

			FRepHandleIterator ArrayHandleIterator(HandleIterator.Owner, HandleIterator.ChangelistIterator, Cmds, ArrayHandleToCmdIndex, Cmd.ElementSize, ArrayNum, HandleIterator.CmdIndex + 1, Cmd.EndCmd - 1);

			check(ArrayHandleIterator.ArrayElementSize> 0);
			check(ArrayHandleIterator.NumHandlesPerElement> 0);

			SendProperties_r(RepState, Writer, bDoChecksum, ArrayHandleIterator, ArrayData, ArrayDepth + 1, SharedInfo, SerializePropertyType);

			check(HandleIterator.ChangelistIterator.ChangedIndex - OldChangedIndex == ArrayChangedCount);				// Make sure we read correct amount
			check(HandleIterator.ChangelistIterator.Changed[HandleIterator.ChangelistIterator.ChangedIndex] == 0);	// Make sure we are at the end

			HandleIterator.ChangelistIterator.ChangedIndex++;

			WritePropertyHandle(Writer, 0, bDoChecksum);		// Signify end of dynamic array
			continue;
		}

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
		if (GDoReplicationContextString > 0 && Writer.PackageMap)
		{
			Writer.PackageMap->SetDebugContextString(FString::Printf(TEXT("%s - %s"), *Owner->GetPathName(), *Cmd.Property->GetPathName()));
		}
#endif

		const FRepSerializedPropertyInfo* SharedPropInfo = nullptr;

		if (bDoSharedSerialization && EnumHasAnyFlags(Cmd.Flags, ERepLayoutCmdFlags::IsSharedSerialization))
		{
			FRepSharedPropertyKey PropertyKey(HandleIterator.CmdIndex, HandleIterator.ArrayIndex, ArrayDepth, (void*)Data.Data);

			SharedPropInfo = SharedInfo->SharedPropertyInfo.FindByPredicate([PropertyKey](const FRepSerializedPropertyInfo& Info)
			{ 
				return (Info.PropertyKey == PropertyKey);
			});
		}

		// Use shared serialization if was found
		if (SharedPropInfo && SharedInfo)
		{
			check(SharedInfo->SerializedProperties.IsValid());

			UE_NET_TRACE_DYNAMIC_NAME_SCOPE(Cmd.Property->GetFName(), Writer, GetTraceCollector(Writer), ENetTraceVerbosity::Trace);
			UE_NET_TRACE_SCOPE(Shared, Writer, GetTraceCollector(Writer), ENetTraceVerbosity::Trace);

			UE_LOGF(LogRepProperties, VeryVerbose, "SerializeProperties_r: SharedSerialization - Handle=%d, Key=%ls", HandleIterator.Handle, *SharedPropInfo->PropertyKey.ToDebugString());
			GNumSharedSerializationHit++;
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
			if (GNetVerifyShareSerializedData != 0)
			{
				FBitWriterMark BitWriterMark(Writer);
			
				UE_LOGF(LogRepProperties, VeryVerbose, "SerializeProperties_r: Verify SharedSerialization, NetSerializeItem");

				WritePropertyHandle(Writer, HandleIterator.Handle, bDoChecksum);
				Cmd.Property->NetSerializeItem(Writer, Writer.PackageMap, const_cast<uint8*>(Data.Data));

#ifdef ENABLE_PROPERTY_CHECKSUMS
				if (bDoChecksum)
				{
					SerializeReadWritePropertyChecksum(Cmd, HandleIterator.CmdIndex, Data, Writer);
				}
#endif
				TArray<uint8> StandardBuffer;
				BitWriterMark.Copy(Writer, StandardBuffer);
				BitWriterMark.Pop(Writer);

				Writer.SerializeBitsWithOffset(SharedInfo->SerializedProperties->GetData(), SharedPropInfo->BitOffset, SharedPropInfo->BitLength);
				
				TArray<uint8> SharedBuffer;
				BitWriterMark.Copy(Writer, SharedBuffer);

				if (StandardBuffer != SharedBuffer)
				{
					UE_LOGF(LogRep, Error, "Shared serialization data mismatch!");
				}
			}
			else
#endif
			{
				Writer.SerializeBitsWithOffset(SharedInfo->SerializedProperties->GetData(), SharedPropInfo->BitOffset, SharedPropInfo->BitLength);
			}

			NETWORK_PROFILER(GNetworkProfiler.TrackReplicateProperty(ParentCmd.Property, SharedPropInfo->PropBitLength, nullptr));
		}
		else
		{
			GNumSharedSerializationMiss++;

			if (SerializePropertyType == ESerializePropertyType::Handle)
			{
				WritePropertyHandle(Writer, HandleIterator.Handle, bDoChecksum);
			}
			else if (SerializePropertyType == ESerializePropertyType::Name)
			{
				WritePropertyName(Writer, Cmd.Property->GetFName(), bDoChecksum);
			}
			else
			{
				UE_LOGF(LogRep, Error, "Unsupported ESerializePropertyType encountered");
			}

			UE_NET_TRACE_DYNAMIC_NAME_SCOPE(Cmd.Property->GetFName(), Writer, GetTraceCollector(Writer), ENetTraceVerbosity::Trace);

			const int32 NumStartBits = Writer.GetNumBits();

			// This property changed, so send it
			Cmd.Property->NetSerializeItem(Writer, Writer.PackageMap, const_cast<uint8*>(Data.Data));
			UE_LOGF(LogRepProperties, VeryVerbose, "SerializeProperties_r: NetSerializeItem");

			const int32 NumEndBits = Writer.GetNumBits();

			NETWORK_PROFILER(GNetworkProfiler.TrackReplicateProperty(ParentCmd.Property, NumEndBits - NumStartBits, nullptr));

#ifdef ENABLE_PROPERTY_CHECKSUMS
			if (bDoChecksum)
			{
				SerializeReadWritePropertyChecksum(Cmd, HandleIterator.CmdIndex, Data, Writer);
			}
#endif
		}

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
		if (GDoReplicationContextString > 0 && Writer.PackageMap)
		{
			Writer.PackageMap->ClearDebugContextString();
		}
#endif
	}
}

void FRepLayout::SendProperties(
	FSendingRepState* RESTRICT RepState,
	FRepChangedPropertyTracker* ChangedTracker,
	const FConstRepObjectDataBuffer Data,
	UClass* ObjectClass,
	FNetBitWriter& Writer,
	TArray<uint16>& Changed,
	const FRepSerializationSharedInfo& SharedInfo,
	const ESerializePropertyType SerializePropertyType) const
{
	SCOPE_CYCLE_COUNTER(STAT_NetReplicateDynamicPropSendTime);

	if (IsEmpty())
	{
		return;
	}

#ifdef ENABLE_PROPERTY_CHECKSUMS
	const bool bDoChecksum = (GDoPropertyChecksum == 1);
#else
	const bool bDoChecksum = false;
#endif

	
	UE_NET_TRACE_SCOPE(Properties, Writer, GetTraceCollector(Writer), ENetTraceVerbosity::Trace);
	FBitWriterMark Mark(Writer);

#ifdef ENABLE_PROPERTY_CHECKSUMS
	Writer.WriteBit(bDoChecksum ? 1 : 0);
#endif

	const int32 NumBits = Writer.GetNumBits();

	UE_LOGF(LogRepProperties, VeryVerbose, "SendProperties: Owner=%ls, LastChangelistIndex=%d", *Owner->GetPathName(), RepState->LastChangelistIndex);

	FChangelistIterator ChangelistIterator(Changed, 0);
	FRepHandleIterator HandleIterator(Owner, ChangelistIterator, Cmds, BaseHandleToCmdIndex, 0, 1, 0, Cmds.Num() - 1);

	SendProperties_r(RepState, Writer, bDoChecksum, HandleIterator, Data, 0, &SharedInfo, SerializePropertyType);

	if (NumBits != Writer.GetNumBits())
	{
		// We actually wrote stuff
		WritePropertyHandle(Writer, 0, bDoChecksum);
	}
	else
	{
		Mark.Pop(Writer);
	}
}

static FORCEINLINE void WritePropertyHandle_BackwardsCompatible(
	FNetBitWriter&	Writer,
	uint32			NetFieldExportHandle,
	bool			bDoChecksum)
{
	UE_NET_TRACE_SCOPE(PropertyHandle, Writer, GetTraceCollector(Writer), ENetTraceVerbosity::Trace);

	const int NumStartingBits = Writer.GetNumBits();

	Writer.SerializeIntPacked(NetFieldExportHandle);
	UE_LOGF(LogRepPropertiesBackCompat, VeryVerbose, "WritePropertyHandle_BackwardsCompatible: %d", NetFieldExportHandle);

#ifdef ENABLE_PROPERTY_CHECKSUMS
	if (bDoChecksum)
	{
		SerializeGenericChecksum(Writer);
	}
#endif

	NETWORK_PROFILER(GNetworkProfiler.TrackWritePropertyHandle(Writer.GetNumBits() - NumStartingBits, nullptr));
}

TSharedPtr<FNetFieldExportGroup> FRepLayout::CreateNetfieldExportGroup() const
{
	TSharedPtr<FNetFieldExportGroup> NetFieldExportGroup = TSharedPtr<FNetFieldExportGroup>(new FNetFieldExportGroup());

	NetFieldExportGroup->PathName = Owner->GetPathName();
	NetFieldExportGroup->NetFieldExports.SetNum(Cmds.Num());

	for (int32 i = 0; i < Cmds.Num(); i++)
	{
		FNetFieldExport NetFieldExport(
			i,
			Cmds[i].CompatibleChecksum,
			Cmds[i].Property ? Cmds[i].Property->GetFName() : NAME_None );

		NetFieldExportGroup->NetFieldExports[i] = NetFieldExport;
	}

	return NetFieldExportGroup;
}

static FORCEINLINE void WriteProperty_BackwardsCompatible(
	FNetBitWriter& Writer,
	const FRepLayoutCmd& Cmd,
	const int32 CmdIndex,
	const UObject* Owner,
	const FConstRepObjectDataBuffer Data,
	const bool bDoChecksum)
{
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	if (GDoReplicationContextString > 0 && Writer.PackageMap)
	{
		Writer.PackageMap->SetDebugContextString(FString::Printf(TEXT("%s - %s"), *Owner->GetPathName(), *Cmd.Property->GetPathName()));
	}
#endif

	const int32 NumStartBits = Writer.GetNumBits();

	FNetBitWriter TempWriter(Writer.PackageMap, 0);

	// This property changed, so send it
	Cmd.Property->NetSerializeItem(TempWriter, TempWriter.PackageMap, const_cast<uint8*>(Data.Data));
	UE_LOGF(LogRepPropertiesBackCompat, VeryVerbose, "WriteProperty_BackwardsCompatible: (Temp) NetSerializeItem");

	uint32 NumBits = TempWriter.GetNumBits();
	Writer.SerializeIntPacked(NumBits);
	Writer.SerializeBits(TempWriter.GetData(), NumBits);
	UE_LOGF(LogRepPropertiesBackCompat, VeryVerbose, "WriteProperty_BackwardsComptaible: Write Temp, NumBits=%d", NumBits);

	const int32 NumEndBits = Writer.GetNumBits();

	NETWORK_PROFILER(GNetworkProfiler.TrackReplicateProperty(Cmd.Property, NumEndBits - NumStartBits, nullptr));

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	if (GDoReplicationContextString > 0 && Writer.PackageMap)
	{
		Writer.PackageMap->ClearDebugContextString();
	}
#endif

#ifdef ENABLE_PROPERTY_CHECKSUMS
	if (bDoChecksum)
	{
		SerializeReadWritePropertyChecksum(Cmd, CmdIndex, Data, Writer);
	}
#endif
}

void FRepLayout::SendProperties_BackwardsCompatible_r(
	FSendingRepState* RESTRICT RepState,
	UPackageMapClient* PackageMapClient,
	FNetFieldExportGroup* NetFieldExportGroup,
	FRepChangedPropertyTracker* ChangedTracker,
	FNetBitWriter& Writer,
	const bool bDoChecksum,
	FRepHandleIterator& HandleIterator,
	const FConstRepObjectDataBuffer SourceData) const
{
	int32 OldIndex = -1;

	FNetBitWriter TempWriter(Writer.PackageMap, 0);

	UE_NET_TRACE_SCOPE(Properties, Writer, GetTraceCollector(Writer), ENetTraceVerbosity::Trace);

	while (HandleIterator.NextHandle())
	{
		const FRepLayoutCmd& Cmd = Cmds[HandleIterator.CmdIndex];
		const FRepParentCmd& ParentCmd = Parents[Cmd.ParentIndex];

		UE_LOGF(LogRepPropertiesBackCompat, VeryVerbose, "SendProperties_BackwardsCompatible_r: Parent=%d, Cmd=%d, ArrayIndex=%d", Cmd.ParentIndex, HandleIterator.CmdIndex, HandleIterator.ArrayIndex);

		FConstRepObjectDataBuffer Data = (SourceData + Cmd) + HandleIterator.ArrayOffset;

		PackageMapClient->TrackNetFieldExport(NetFieldExportGroup, HandleIterator.CmdIndex);

		if (HandleIterator.ArrayElementSize> 0 && HandleIterator.ArrayIndex != OldIndex)
		{
			if (OldIndex != -1)
			{
				WritePropertyHandle_BackwardsCompatible(Writer, 0, bDoChecksum);
			}

			UE_LOGF(LogRepPropertiesBackCompat, VeryVerbose, "SendProperties_BackwardsCompatible_r: WriteArrayIndex=%d", HandleIterator.ArrayIndex)
			uint32 Index = HandleIterator.ArrayIndex + 1;
			Writer.SerializeIntPacked(Index);
			OldIndex = HandleIterator.ArrayIndex;
		}

		WritePropertyHandle_BackwardsCompatible(Writer, HandleIterator.CmdIndex + 1, bDoChecksum);

		UE_NET_TRACE_DYNAMIC_NAME_SCOPE(Cmd.Property->GetFName(), Writer, GetTraceCollector(Writer), ENetTraceVerbosity::Trace);

		if (Cmd.Type == ERepLayoutCmdType::DynamicArray)
		{
			const FScriptArray* Array = (FScriptArray *)Data.Data;
			const FConstRepObjectDataBuffer ArrayData(Array->GetData());

			uint32 ArrayNum = Array->Num();

			// Read the jump offset
			// We won't need to actually jump over anything because we expect the change list to be pruned once we get here
			// But we can use it to verify we read the correct amount.
			const int32 ArrayChangedCount = HandleIterator.ChangelistIterator.Changed[HandleIterator.ChangelistIterator.ChangedIndex++];

			const int32 OldChangedIndex = HandleIterator.ChangelistIterator.ChangedIndex;

			TArray<FHandleToCmdIndex>& ArrayHandleToCmdIndex = *HandleIterator.HandleToCmdIndex[Cmd.RelativeHandle - 1].HandleToCmdIndex;

			FRepHandleIterator ArrayHandleIterator(HandleIterator.Owner, HandleIterator.ChangelistIterator, Cmds, ArrayHandleToCmdIndex, Cmd.ElementSize, ArrayNum, HandleIterator.CmdIndex + 1, Cmd.EndCmd - 1);

			check(ArrayHandleIterator.ArrayElementSize> 0);
			check(ArrayHandleIterator.NumHandlesPerElement> 0);

			TempWriter.Reset();

			// Write array num
			UE_LOGF(LogRepPropertiesBackCompat, VeryVerbose, "SendProperties_BackwardsCompatible_r: (Temp) ArrayNum=%d", ArrayNum);
			TempWriter.SerializeIntPacked(ArrayNum);

			if (ArrayNum> 0)
			{
				UE_LOGF(LogRepPropertiesBackCompat, VeryVerbose, "SendProperties_BackwardsCompatible_r: (Temp) Array Recurse Properties");
				SendProperties_BackwardsCompatible_r(RepState, PackageMapClient, NetFieldExportGroup, ChangedTracker, TempWriter, bDoChecksum, ArrayHandleIterator, ArrayData);
			}

			uint32 EndArrayIndex = 0;
			TempWriter.SerializeIntPacked(EndArrayIndex);
			UE_LOGF(LogRepPropertiesBackCompat, VeryVerbose, "SendProperties_BackwardsCompatible_r: (Temp) Array Footer");

			uint32 NumBits = TempWriter.GetNumBits();
			Writer.SerializeIntPacked(NumBits);
			Writer.SerializeBits(TempWriter.GetData(), NumBits);
			UE_LOGF(LogRepPropertiesBackCompat, VeryVerbose, "SendProperties_BackwardsCompatible_r: Write Temp, NumBits=%d", NumBits);

			check(HandleIterator.ChangelistIterator.ChangedIndex - OldChangedIndex == ArrayChangedCount);				// Make sure we read correct amount
			check(HandleIterator.ChangelistIterator.Changed[HandleIterator.ChangelistIterator.ChangedIndex] == 0);	// Make sure we are at the end

			HandleIterator.ChangelistIterator.ChangedIndex++;
			continue;
		}

		WriteProperty_BackwardsCompatible(Writer, Cmd, HandleIterator.CmdIndex, Owner, Data, bDoChecksum);
	}

	WritePropertyHandle_BackwardsCompatible(Writer, 0, bDoChecksum);
}

void FRepLayout::SendAllProperties_BackwardsCompatible_r(
	FSendingRepState* RESTRICT RepState,
	FNetBitWriter& Writer,
	const bool bDoChecksum,
	UPackageMapClient* PackageMapClient,
	FNetFieldExportGroup* NetFieldExportGroup,
	const int32 CmdStart,
	const int32 CmdEnd, 
	const FConstRepObjectDataBuffer SourceData) const
{
	FNetBitWriter TempWriter(Writer.PackageMap, 0);

	for (int32 CmdIndex = CmdStart; CmdIndex < CmdEnd; CmdIndex++)
	{
		const FRepLayoutCmd& Cmd = Cmds[CmdIndex];

		UE_LOGF(LogRepPropertiesBackCompat, VeryVerbose, "SendAllProperties_BackwardsCompatible_r: Parent=%d, Cmd=%d", Cmd.ParentIndex, CmdIndex);

		check(Cmd.Type != ERepLayoutCmdType::Return);

		const bool bIsArray = Cmd.Type == ERepLayoutCmdType::DynamicArray;
		if (bIsArray && EnumHasAnyFlags(Cmd.Flags, ERepLayoutCmdFlags::IsEmptyArrayStruct))
		{
			CmdIndex = Cmd.EndCmd - 1;
			continue;
		}

		PackageMapClient->TrackNetFieldExport(NetFieldExportGroup, CmdIndex);
		WritePropertyHandle_BackwardsCompatible(Writer, CmdIndex + 1, bDoChecksum);

		FConstRepObjectDataBuffer Data = SourceData + Cmd;

		if (bIsArray)
		{			
			const FScriptArray* Array = (FScriptArray *)Data.Data;
			const FConstRepObjectDataBuffer ArrayData(Array->GetData());

			TempWriter.Reset();

			// Write array num
			uint32 ArrayNum = Array->Num();
			TempWriter.SerializeIntPacked(ArrayNum);

			UE_LOGF(LogRepPropertiesBackCompat, VeryVerbose, "SendAllProperties_BackwardsCompatible_r: (Temp) ArrayNum=%d", ArrayNum);

			for (int32 i = 0; i < Array->Num(); i++)
			{
				uint32 ArrayIndex = i + 1;
				TempWriter.SerializeIntPacked(ArrayIndex);

				UE_LOGF(LogRepPropertiesBackCompat, VeryVerbose, "SendAllProperties_BackwardsCompatible_r: (Temp) ArrayIndex=%d", ArrayIndex);
				const int32 ArrayElementOffset = Cmd.ElementSize * i;
				SendAllProperties_BackwardsCompatible_r(RepState, TempWriter, bDoChecksum, PackageMapClient, NetFieldExportGroup, CmdIndex + 1, Cmd.EndCmd - 1, ArrayData + ArrayElementOffset);
			}

			uint32 EndArrayIndex = 0;
			TempWriter.SerializeIntPacked(EndArrayIndex);
			UE_LOGF(LogRepPropertiesBackCompat, VeryVerbose, "SendAllProperties_BackwardsCompatible_r: (Temp) ArrayFooter");

			uint32 NumBits = TempWriter.GetNumBits();
			Writer.SerializeIntPacked(NumBits);
			Writer.SerializeBits(TempWriter.GetData(), NumBits);
			UE_LOGF(LogRepPropertiesBackCompat, VeryVerbose, "SendAllProperties_BackwardsCompatible_r: Write Temp, NumBits=%d", NumBits);

			CmdIndex = Cmd.EndCmd - 1;		// The -1 to handle the ++ in the for loop
			continue;
		}

		WriteProperty_BackwardsCompatible(Writer, Cmd, CmdIndex, Owner, Data, bDoChecksum);
	}

	WritePropertyHandle_BackwardsCompatible(Writer, 0, bDoChecksum);
}

void FRepLayout::SendProperties_BackwardsCompatible(
	FSendingRepState* RESTRICT RepState,
	FRepChangedPropertyTracker* ChangedTracker,
	const FConstRepObjectDataBuffer Data,
	UNetConnection* Connection,
	FNetBitWriter& Writer,
	TArray<uint16>& Changed) const
{
	SCOPE_CYCLE_COUNTER(STAT_NetReplicateDynamicPropSendBackCompatTime);

	FBitWriterMark Mark(Writer);

#ifdef ENABLE_PROPERTY_CHECKSUMS
	const bool bDoChecksum = (GDoPropertyChecksum == 1);
	Writer.WriteBit(bDoChecksum ? 1 : 0);
#else
	const bool bDoChecksum = false;
#endif

	UPackageMapClient* PackageMapClient = (UPackageMapClient*)Connection->PackageMap;
	const FString OwnerPathName = Owner->GetPathName();
	UE_LOGF(LogRepPropertiesBackCompat, VeryVerbose, "SendProperties_BackwardsCompatible: Owner=%ls, LastChangelistIndex=%d", *OwnerPathName, RepState ? RepState->LastChangelistIndex : INDEX_NONE);

	TSharedPtr<FNetFieldExportGroup> NetFieldExportGroup = PackageMapClient->GetNetFieldExportGroup(OwnerPathName);

	if (!NetFieldExportGroup.IsValid())
	{
		UE_LOGF(LogRepPropertiesBackCompat, VeryVerbose, "SendProperties_BackwardsCompatible: Create Netfield Export Group.")
		NetFieldExportGroup = CreateNetfieldExportGroup();

		PackageMapClient->AddNetFieldExportGroup(OwnerPathName, NetFieldExportGroup);
	}

	checkf(NetFieldExportGroup->NetFieldExports.Num() == Cmds.Num(),
		TEXT("NetFieldExports.Num() does not match number of commands! PathName = %s, NetFieldExportGroup.PathName = %s, Cmds.Num() = %d, NetFieldExports.Num() = %d"),
		*OwnerPathName, *(NetFieldExportGroup->PathName), Cmds.Num(), NetFieldExportGroup->NetFieldExports.Num());

	const int32 NumBits = Writer.GetNumBits();

	if (Changed.Num() == 0)
	{
		UE_LOGF(LogRepPropertiesBackCompat, VeryVerbose, "SendProperties_BackwardsCompatible: SendAllProperties.");
		SendAllProperties_BackwardsCompatible_r(RepState, Writer, bDoChecksum, PackageMapClient, NetFieldExportGroup.Get(), 0, Cmds.Num() - 1, Data);
	}
	else
	{
		UE_LOGF(LogRepPropertiesBackCompat, VeryVerbose, "SendProperties_BackwardsCompatible: SendProperties.");
		FChangelistIterator ChangelistIterator(Changed, 0);
		FRepHandleIterator HandleIterator(Owner, ChangelistIterator, Cmds, BaseHandleToCmdIndex, 0, 1, 0, Cmds.Num() - 1);

		SendProperties_BackwardsCompatible_r(RepState, PackageMapClient, NetFieldExportGroup.Get(), ChangedTracker, Writer, bDoChecksum, HandleIterator, Data);
	}

	if (NumBits == Writer.GetNumBits())
	{
		Mark.Pop(Writer);
	}
}

static bool ReceivePropertyHelper(
	FNetBitReader& Bunch, 
	FGuidReferencesMap* GuidReferencesMap,
	const int32 ElementOffset, 
	FRepShadowDataBuffer ShadowData,
	FRepObjectDataBuffer Data,
	TArray<FProperty*>* RepNotifies,
	const bool bShadowDataCopied,
	const TArray<FRepParentCmd>& Parents,
	const TArray<FRepLayoutCmd>& Cmds,
	const int32 CmdIndex,
	const bool bDoChecksum,
	bool& bOutGuidsChanged,
	const bool bSkipSwapRoles,
	const TMap<FRepLayoutCmd*, TArray<FRepLayoutCmd>>& NetSerializeLayouts,
	const UObject* OwningObject)
{
	const FRepLayoutCmd& Cmd = Cmds[CmdIndex];
	const FRepParentCmd& Parent = Parents[Cmd.ParentIndex];

	auto GetSwappedCmd = [&Cmd, &Cmds, &Parents, bSkipSwapRoles]() -> const FRepLayoutCmd&
	{
		if (!bSkipSwapRoles)
		{
			// Swap Role to RemoteRole, and vice-versa. Leave everything else the same.
			if (UNLIKELY((int32)AActor::ENetFields_Private::RemoteRole == Cmd.ParentIndex))
			{
				return Cmds[Parents[(int32)AActor::ENetFields_Private::Role].CmdStart];
			}
			else if (UNLIKELY((int32)AActor::ENetFields_Private::Role == Cmd.ParentIndex))
			{
				return Cmds[Parents[(int32)AActor::ENetFields_Private::RemoteRole].CmdStart];
			}
		}

		return Cmd;
	};

	// This swaps Role/RemoteRole as we write it
	const FRepLayoutCmd& SwappedCmd = GetSwappedCmd();

	if (GuidReferencesMap)		// Don't reset unmapped guids here if we are told not to (assuming calling code is handling this)
	{
		// Let package map know we want to track and know about any guids that are unmapped during the serialize call
		Bunch.PackageMap->ResetTrackedGuids(true);
	}

	Bunch.PackageMap->ResetTrackedSyncLoadedGuids();

	// Remember where we started reading from, so that if we have unmapped properties, we can re-deserialize from this data later
	FBitReaderMark Mark(Bunch);

	if (RepNotifies != nullptr && INDEX_NONE != Parent.RepNotifyNumParams)
	{
		// Copy current value over so we can check to see if it changed
		if (!bShadowDataCopied)
		{
			StoreProperty(Cmd, ShadowData + Cmd, Data + SwappedCmd);
		}

		// Read the property
		Cmd.Property->NetSerializeItem(Bunch, Bunch.PackageMap, Data + SwappedCmd);
		UE_LOGF(LogRepProperties, VeryVerbose, "ReceivePropertyHelper: NetSerializeItem (WithRepNotify)");

		// Check to see if this property changed
		if (Parent.RepNotifyCondition == REPNOTIFY_Always || !PropertiesAreIdentical(Cmd, ShadowData + Cmd, Data + SwappedCmd, NetSerializeLayouts))
		{
			RepNotifies->AddUnique(Parent.Property);
		}
		else
		{
			UE_CLOGF(LogSkippedRepNotifies > 0, LogRep, Display, "2 FReceivedPropertiesStackState Skipping RepNotify for property %ls because local value has not changed.", *Cmd.Property->GetName());
		}
	}
	else
	{
		Cmd.Property->NetSerializeItem(Bunch, Bunch.PackageMap, Data + SwappedCmd);
		UE_LOGF(LogRepProperties, VeryVerbose, "ReceivePropertyHelper: NetSerializeItem (WithoutRepNotify)");
	}

#ifdef ENABLE_PROPERTY_CHECKSUMS
	if (bDoChecksum)
	{
		SerializeReadWritePropertyChecksum(Cmd, CmdIndex, FConstRepObjectDataBuffer(Data + SwappedCmd), Bunch);
	}
#endif

	Bunch.PackageMap->ReportSyncLoadsForProperty(Cmd.Property, OwningObject);

	if (GuidReferencesMap)
	{
		const int32 AbsOffset = ElementOffset + SwappedCmd.Offset;

		// Loop over all de-serialized network guids and track them so we can manage their pointers as their replicated reference goes in/out of relevancy
		const TSet<FNetworkGUID>& TrackedUnmappedGuids = Bunch.PackageMap->GetTrackedUnmappedGuids();
		const TSet<FNetworkGUID>& TrackedDynamicMappedGuids = Bunch.PackageMap->GetTrackedDynamicMappedGuids();

		const bool bHasUnmapped = TrackedUnmappedGuids.Num()> 0;

		FGuidReferences* GuidReferences = GuidReferencesMap->Find(AbsOffset);

		if (TrackedUnmappedGuids.Num() > 0 || TrackedDynamicMappedGuids.Num()> 0)
		{
			if (GuidReferences != nullptr)
			{
				check(GuidReferences->CmdIndex == CmdIndex);
				check(GuidReferences->ParentIndex == Cmd.ParentIndex);

				// If we're already tracking the guids, re-copy lists only if they've changed
				if (!NetworkGuidSetsAreSame(GuidReferences->GetUnmappedGUIDs(), TrackedUnmappedGuids))
				{
					bOutGuidsChanged = true;
				}
				else if (!NetworkGuidSetsAreSame(GuidReferences->MappedDynamicGUIDs, TrackedDynamicMappedGuids))
				{
					bOutGuidsChanged = true;
				}
			}
			
			if (GuidReferences == nullptr || bOutGuidsChanged)
			{
				// First time tracking these guids (or guids changed), so add (or replace) new entry
				GuidReferencesMap->Emplace(AbsOffset, FGuidReferences(Bunch, Mark, TrackedUnmappedGuids, TrackedDynamicMappedGuids, Cmd.ParentIndex, CmdIndex, Bunch.PackageMap));
				bOutGuidsChanged = true;
			}
			else if (UE::Net::Private::bAlwaysUpdateGuidReferenceMapForNetSerializeObjectStruct && Cmd.Type == ERepLayoutCmdType::NetSerializeStructWithObjectReferences)
			{
				// If this is a NetSerialize struct with object references, there may be other properties "wrapped up" with this GUID reference.
				// In this case, the entry in the map should be always be updated, so there isn't outdated data in the entry that also gets
				// applied when the Guid possibly goes unmapped and then mapped later.
				GuidReferencesMap->Emplace(AbsOffset, FGuidReferences(Bunch, Mark, TrackedUnmappedGuids, TrackedDynamicMappedGuids, Cmd.ParentIndex, CmdIndex, Bunch.PackageMap));
			}
		}
		else
		{
			// If we don't have any unmapped guids, then make sure to remove the entry so we don't serialize old data when we update unmapped objects
			if (GuidReferences != nullptr)
			{
				GuidReferencesMap->Remove(AbsOffset);
				bOutGuidsChanged = true;
			}
		}

		// Stop tracking unmapped objects
		Bunch.PackageMap->ResetTrackedGuids(false);

		return bHasUnmapped;
	}

	return false;
}

static FGuidReferencesMap* PrepReceivedArray(
	const int32 ArrayNum,
	FScriptArray* ShadowArray,
	FScriptArray* DataArray,
	FGuidReferencesMap* ParentGuidReferences,
	const int32 AbsOffset,
	const FRepParentCmd& Parent, 
	const FRepLayoutCmd& Cmd, 
	const int32 CmdIndex,
	FRepShadowDataBuffer* OutShadowBaseData,
	FRepObjectDataBuffer* OutBaseData,
	TArray<FProperty*>* RepNotifies,
	bool& bOutShadowDataCopied,
	UPackageMap* PackageMap)
{
	FGuidReferences* NewGuidReferencesArray = nullptr;

	if (ParentGuidReferences != nullptr)
	{
		// Since we don't know yet if something under us could be unmapped, go ahead and allocate an array container now
		NewGuidReferencesArray = ParentGuidReferences->Find(AbsOffset);

		if (NewGuidReferencesArray == nullptr)
		{
			NewGuidReferencesArray = &ParentGuidReferences->Emplace(AbsOffset, FGuidReferences(new FGuidReferencesMap, Cmd.ParentIndex, CmdIndex, PackageMap));
		}

		check(NewGuidReferencesArray != nullptr);
		check(NewGuidReferencesArray->ParentIndex == Cmd.ParentIndex);
		check(NewGuidReferencesArray->CmdIndex == CmdIndex);
	}

	if (RepNotifies && Parent.RepNotifyNumParams != INDEX_NONE)
	{
		if (DataArray->Num() != ArrayNum || Parent.RepNotifyCondition == REPNOTIFY_Always)
		{
			(*RepNotifies).AddUnique(Parent.Property);
		}
		else
		{
			UE_CLOGF(LogSkippedRepNotifies > 0, LogRep, Display, "1 FReceivedPropertiesStackState Skipping RepNotify for property %ls because local value has not changed.", *Cmd.Property->GetName());
		}
		check(ShadowArray != nullptr);

		// If a top level property already set the current data in ShadowBuffer, we don't need to redo it again.
		if (!bOutShadowDataCopied)
		{
			// Does the OnRep function have a parameter to receive the previous version of the array
			if (Parent.RepNotifyNumParams > 0)
			{
				// Copy the entire array into the shadow buffer before it gets overwritten by the network data.
				// The OnRep callback will pass that array back as a function parameter and it needs to be the current local array before the network data was applied.
				Cmd.Property->CopyCompleteValue((uint8*)ShadowArray, (uint8*)DataArray);
				bOutShadowDataCopied = true;
			}
			else if (ShadowArray->Num() != DataArray->Num())
			{
				// When individual entries get netserialized, they will copy over the current entry into the shadow buffer so ensure the array has the size to do so. 
				FScriptArrayHelper ShadowArrayHelper((FArrayProperty*)Cmd.Property, ShadowArray);
				ShadowArrayHelper.Resize(DataArray->Num());
			}
		}

		*OutShadowBaseData = ShadowArray->GetData();
	}
	else
	{
		*OutShadowBaseData = nullptr;
	}

	check(CastFieldChecked<FArrayProperty>(Cmd.Property) != nullptr);

	// Resize arrays if needed
	FScriptArrayHelper ArrayHelper((FArrayProperty*)Cmd.Property, DataArray);
	ArrayHelper.Resize(ArrayNum);

	// Re-compute the base data values since they could have changed after the resize above
	*OutBaseData = DataArray->GetData();

	return NewGuidReferencesArray ? NewGuidReferencesArray->Array : nullptr;
}

/** Struct containing parameters that don't change or can be shared throughout recursion of ReceiveProperties_r */
struct FReceivePropertiesSharedParams
{
	const bool bDoChecksum;
	const bool bSkipRoleSwap;
	FNetBitReader& Bunch;
	bool& bOutHasUnmapped;
	bool& bOutGuidsChanged;
	const TArray<FRepParentCmd>& Parents;
	const TArray<FRepLayoutCmd>& Cmds;
	const TMap<FRepLayoutCmd*, TArray<FRepLayoutCmd>> NetSerializeLayouts;
	UObject* OwningObject;
	FNetTraceCollector* TraceCollector = nullptr;
	uint16 ReadHandle = 0;
	
#if WITH_PUSH_MODEL	
	int32 LastReceivedParent = INDEX_NONE;
#endif	
};

/** Struct containing parameters that do change or can't be shared as we recurse into ReceiveProperties_r */
struct FReceivePropertiesStackParams
{
	FRepObjectDataBuffer ObjectData;
	FRepShadowDataBuffer ShadowData;
	FGuidReferencesMap* GuidReferences;
	const int32 CmdStart;
	const int32 CmdEnd;
	TArray<FProperty*>* RepNotifies;
	uint32 ArrayElementOffset = 0;
	uint16 CurrentHandle = 0;
    bool bShadowDataCopied = false;
};

static FORCEINLINE void ReadPropertyHandle(FReceivePropertiesSharedParams& Params)
{
	UE_NET_TRACE_SCOPE(PropertyHandle, Params.Bunch, Params.TraceCollector, ENetTraceVerbosity::Trace);

	uint32 Handle = 0;
	Params.Bunch.SerializeIntPacked(Handle);

	UE_LOGF(LogRepProperties, VeryVerbose, "ReadPropertyHandle: Handle=%d", Handle);

#ifdef ENABLE_PROPERTY_CHECKSUMS
	if (Params.bDoChecksum)
	{
		SerializeGenericChecksum(Params.Bunch);
	}
#endif

	Params.ReadHandle = Handle;
}

static bool ReceiveProperties_r(FReceivePropertiesSharedParams& Params, FReceivePropertiesStackParams& StackParams)
{
	// Note, it's never possible for the ObjectData to be nullptr.
	// However, it is possible for the ShadowData to be nullptr.
	// At the top level, ShadowData will always be valid.
	// If RepNotifies aren't being used, PrepReceivedArray will ignore the current shadow data and just null out the next level's shadow data.
	// If RepNotifies aren't being used, ReceivePropertyHelper will ignore the shadow data.

	check(StackParams.GuidReferences != nullptr);
	for (int32 CmdIndex = StackParams.CmdStart; CmdIndex < StackParams.CmdEnd; ++CmdIndex)
	{
		const FRepLayoutCmd& Cmd = Params.Cmds[CmdIndex];
		check(ERepLayoutCmdType::Return != Cmd.Type);

		++StackParams.CurrentHandle;
		if (StackParams.CurrentHandle != Params.ReadHandle)
		{
			// Skip this property.
			if (ERepLayoutCmdType::DynamicArray == Cmd.Type)
			{
				CmdIndex = Cmd.EndCmd - 1;
			}

			UE_LOGF(LogRepProperties, VeryVerbose, "ReceiveProperties_r: Skipping Property Parent=%d, Cmd=%d, CurrentHandle=%d, ReadHandle=%d",
				Cmd.ParentIndex, CmdIndex, StackParams.CurrentHandle, Params.ReadHandle);
		}
		else
		{
			UE_LOGF(LogRepProperties, VeryVerbose, "ReceiveProperties_r: Parent=%d, Cmd=%d", Cmd.ParentIndex, CmdIndex);

			const FRepParentCmd& Parent = Params.Parents[Cmd.ParentIndex];

#if WITH_PUSH_MODEL
			if (Cmd.ParentIndex != Params.LastReceivedParent)
			{
				// Make sure we mark received properties dirty.
				// This will make sure that anything we receive can be picked up by other Net Drivers
				// (like the DemoNetDriver) on clients.
				Params.LastReceivedParent = Cmd.ParentIndex;
				FNetPrivatePushIdHelper::MarkPropertyDirty(Params.OwningObject, Params.LastReceivedParent);
			}
#endif

			if (ERepLayoutCmdType::DynamicArray == Cmd.Type)
			{
				UE_NET_TRACE_SCOPE(DynamicArray, Params.Bunch, Params.TraceCollector,  ENetTraceVerbosity::Trace);

				// Don't worry about checking the ShadowData for nullptr here.
				// We're either:
				//	1. At the top level and it's valid
				//	2. Tracking RepNotifies and it's valid.
				//	3. We aren't tracking RepNotifies in which case it will be ignored.
				FScriptArray* ShadowArray = (FScriptArray*)(StackParams.ShadowData + Cmd).Data;
				FScriptArray* ObjectArray = (FScriptArray*)(StackParams.ObjectData + Cmd).Data;

				// Setup a new Stack State for our array.
				FReceivePropertiesStackParams ArrayStackParams{
					nullptr,
					nullptr,
					nullptr,
					CmdIndex + 1,
					Cmd.EndCmd - 1,
					StackParams.RepNotifies,
					0 /*ArrayElementOffset*/,
					0 /*CurrentHandle*/,
					StackParams.bShadowDataCopied
				};

				// These buffers will track the dynamic array memory.
				FRepObjectDataBuffer ObjectArrayBuffer = StackParams.ObjectData;
				FRepShadowDataBuffer ShadowArrayBuffer = StackParams.ShadowData;

				// Read the number of elements in the array, and resize as necessary.
				uint16 ArrayNum = 0;
				Params.Bunch << ArrayNum;

				UE_LOGF(LogRepProperties, VeryVerbose, "ReceiveProperties_r: ArrayNum=%d", ArrayNum);

				ArrayStackParams.GuidReferences = PrepReceivedArray(
					ArrayNum,
					ShadowArray,
					ObjectArray,
					StackParams.GuidReferences,

					// We pass in the ArrayElementOffset + Offset here, because PrepReceivedArray expects the absolute offset.
					StackParams.ArrayElementOffset + Cmd.Offset,
					Params.Parents[Cmd.ParentIndex],
					Cmd,
					CmdIndex,
					&ShadowArrayBuffer,
					&ObjectArrayBuffer,
					StackParams.RepNotifies,
					ArrayStackParams.bShadowDataCopied,
					Params.Bunch.PackageMap);

				// Read the next array handle.
				ReadPropertyHandle(Params);

				// It's possible that we've already hit the terminator.
				// Maybe this was just a change of size of the array (like removing an element from the end).
				if (0 != Params.ReadHandle)
				{
					const int32 ObjectArrayNum = ObjectArray->Num();
					for (int32 i = 0; i < ObjectArrayNum; ++i)
					{
						const int32 ElementOffset = i * Cmd.ElementSize;

						ArrayStackParams.ObjectData = ObjectArrayBuffer + ElementOffset;
						ArrayStackParams.ArrayElementOffset = ElementOffset;

						// If ShadowArrayBuffer is valid, then we know that our ShadowArray pointer is also valid and pointing to a valid array.
						// So we just need to make sure we're not going outside the bounds of the array.
						ArrayStackParams.ShadowData = (ShadowArrayBuffer && i < ShadowArray->Num()) ? (ShadowArrayBuffer + ElementOffset) : nullptr;
						ArrayStackParams.RepNotifies = ArrayStackParams.ShadowData ? StackParams.RepNotifies : nullptr;

						UE_LOGF(LogRepProperties, VeryVerbose, "ReceivePropertiesArray_r: Recursing - Parent=%d, Cmd=%d, Index=%d", Cmd.ParentIndex, CmdIndex, i);
						if (!ReceiveProperties_r(Params, ArrayStackParams))
						{
							UE_LOGF(LogRep, Error, "ReceiveProperties_r: Failed to receive property, Array Property - Property=%ls, Parent=%d, Cmd=%d, Index=%d", *Parent.CachedPropertyName.ToString(), Cmd.ParentIndex, CmdIndex, i);
							return false;
						}
					}

					// Make sure we've hit the array terminator.
					if (0 != Params.ReadHandle)
					{
						UE_LOGF(LogRep, Warning, "ReceiveProperties_r: Failed to receive property, Array Property Improperly Terminated - Property=%ls, Parent=%d, CmdIndex=%d, ReadHandle=%d", *Parent.CachedPropertyName.ToString(), Cmd.ParentIndex, CmdIndex, Params.ReadHandle);
						return false;
					}
				}

				// Skip passed the inner array properties.
				CmdIndex = Cmd.EndCmd - 1;
			}
			else
			{
				UE_NET_TRACE_DYNAMIC_NAME_SCOPE(Cmd.Property->GetFName(), Params.Bunch, Params.TraceCollector, ENetTraceVerbosity::Trace);

				// Go ahead and receive the property.
				if (ReceivePropertyHelper(
					Params.Bunch,
					StackParams.GuidReferences,
					StackParams.ArrayElementOffset,
					StackParams.ShadowData,
					StackParams.ObjectData,
					StackParams.RepNotifies,
					StackParams.bShadowDataCopied,
					Params.Parents,
					Params.Cmds,
					CmdIndex,
					Params.bDoChecksum,
					Params.bOutGuidsChanged,
					Params.bSkipRoleSwap,
					Params.NetSerializeLayouts,
					Params.OwningObject))
				{
					Params.bOutHasUnmapped = true;
				}
			}

			// TODO: Might be worth doing this before and after ReadNextHandle, or having ReadNextHandle check for errors?
			if (Params.Bunch.IsError())
			{
				UE_LOGF(LogRep, Error, "ReceiveProperties_r: Failed to receive property, BunchIsError - Property=%ls, Parent=%d, Cmd=%d, ReadHandle=%d", *Parent.CachedPropertyName.ToString(), Cmd.ParentIndex, CmdIndex, Params.ReadHandle);
				return false;
			}

			// Read the next property handle to serialize.
			// If we don't have any more properties, this could be a terminator.
			ReadPropertyHandle(Params);

			if (Params.ReadHandle != 0 && StackParams.CurrentHandle > Params.ReadHandle)
			{
				// Serialization of this property possibly has a bug and corrupted state, causing an invalid handle value to be read.
				UE_LOGF(LogRep, Error, "Replicated property %ls has likely corrupted serialization of %ls. Check its serialization code.", *Cmd.Property->GetFullName(), *GetFullNameSafe(Params.OwningObject));
			}
		}
	}

	return true;
}

bool FRepLayout::ReceiveProperties(
	UActorChannel* OwningChannel,
	UClass* InObjectClass,
	FReceivingRepState* RESTRICT RepState,
	UObject* Object,
	FNetBitReader& InBunch,
	bool& bOutHasUnmapped,
	bool& bOutGuidsChanged,
	const EReceivePropertiesFlags ReceiveFlags) const
{
	check(InObjectClass == Owner);

	FRepObjectDataBuffer Data(Object);
	const bool bEnableRepNotifies = EnumHasAnyFlags(ReceiveFlags, EReceivePropertiesFlags::RepNotifies);

	UE_NET_TRACE_SCOPE(Properties, InBunch, OwningChannel->Connection->GetInTraceCollector(), ENetTraceVerbosity::Trace);

	if (OwningChannel->Connection->IsInternalAck())
	{
		return ReceiveProperties_BackwardsCompatible(OwningChannel->Connection, RepState, Data, InBunch, bOutHasUnmapped, bEnableRepNotifies, bOutGuidsChanged, Object);
	}

#ifdef ENABLE_PROPERTY_CHECKSUMS
	const bool bDoChecksum = InBunch.ReadBit() ? true : false;
#else
	const bool bDoChecksum = false;
#endif

	UE_LOGF(LogRepProperties, VeryVerbose, "ReceiveProperties: Owner=%ls", *Owner->GetPathName());

	bOutHasUnmapped = false;

	// If we've gotten this far, it means that the server must have sent us something.
	// That should only happen if there's actually commands to process.
	// If this is hit, it may mean the Client and Server have different properties!
	check(!IsEmpty());

	FReceivePropertiesSharedParams Params{
		bDoChecksum,
		// We can skip swapping roles if we're not an Actor layout, or if we've been explicitly told we can skip.
		EnumHasAnyFlags(ReceiveFlags, EReceivePropertiesFlags::SkipRoleSwap) || !EnumHasAnyFlags(Flags, ERepLayoutFlags::IsActor),
		InBunch,
		bOutHasUnmapped,
		bOutGuidsChanged,
		Parents,
		Cmds,
		NetSerializeLayouts,
		Object,
		OwningChannel->Connection->GetInTraceCollector()
	};

	FReceivePropertiesStackParams StackParams{
		FRepObjectDataBuffer(Data),
		FRepShadowDataBuffer(RepState->StaticBuffer.GetData()),
		&RepState->GuidReferencesMap,
		0,
		Cmds.Num() - 1,
		bEnableRepNotifies ? &RepState->RepNotifies : nullptr
	};

	// Read the first handle, and then start receiving properties.
	ReadPropertyHandle(Params);
	if (ReceiveProperties_r(Params, StackParams))
	{
		if (0 != Params.ReadHandle)
		{
			UE_LOGF(LogRep, Error, "ReceiveProperties: Invalid property terminator handle - Handle=%d", Params.ReadHandle);
			return false;
		}

#ifdef ENABLE_SUPER_CHECKSUMS
		if (bDoChecksum)
		{
			ValidateWithChecksum<>(FConstRepShadowDataBuffer(RepState->StaticBuffer.GetData()), InBunch);
		}
#endif

		return true;
	}

	return false;
}

bool FRepLayout::ReceiveProperties_BackwardsCompatible(
	UNetConnection* Connection,
	FReceivingRepState* RESTRICT RepState,
	FRepObjectDataBuffer Data,
	FNetBitReader& InBunch,
	bool& bOutHasUnmapped,
	const bool bEnableRepNotifies,
	bool& bOutGuidsChanged,
	UObject* OwningObject) const
{
#ifdef ENABLE_PROPERTY_CHECKSUMS
	const bool bDoChecksum = InBunch.ReadBit() ? true : false;
#else
	const bool bDoChecksum = false;
#endif

	bOutHasUnmapped = false;

	const FString OwnerPathName = Owner->GetPathName();
	TSharedPtr<FNetFieldExportGroup> NetFieldExportGroup = ((UPackageMapClient*)Connection->PackageMap)->GetNetFieldExportGroup(OwnerPathName);

	UE_LOGF(LogRepPropertiesBackCompat, VeryVerbose, "ReceiveProperties_BackwardsCompatible: Owner=%ls, NetFieldExportGroupFound=%d", *OwnerPathName, !!NetFieldExportGroup.IsValid());

	return ReceiveProperties_BackwardsCompatible_r(RepState, NetFieldExportGroup.Get(), InBunch, 0, Cmds.Num() - 1, (bEnableRepNotifies && RepState) ? RepState->StaticBuffer.GetData() : nullptr, Data, Data, RepState ? &RepState->GuidReferencesMap : nullptr, bOutHasUnmapped, bOutGuidsChanged, OwningObject);
}

int32 FRepLayout::FindCompatibleProperty(
	const int32		CmdStart,
	const int32		CmdEnd,
	const uint32	Checksum) const
{
	for (int32 CmdIndex = CmdStart; CmdIndex < CmdEnd; CmdIndex++)
	{
		const FRepLayoutCmd& Cmd = Cmds[CmdIndex];

		check(Cmd.Type != ERepLayoutCmdType::Return);

		if (Cmd.CompatibleChecksum == Checksum)
		{
			return CmdIndex;
		}

		// Jump over entire array and inner properties if checksum didn't match
		if (Cmd.Type == ERepLayoutCmdType::DynamicArray)
		{
			CmdIndex = Cmd.EndCmd - 1;
		}
	}

	return -1;
}

bool FRepLayout::ReceiveProperties_BackwardsCompatible_r(
	FReceivingRepState* RESTRICT RepState,
	FNetFieldExportGroup* NetFieldExportGroup,
	FNetBitReader& Reader,
	const int32 CmdStart,
	const int32 CmdEnd,
	FRepShadowDataBuffer ShadowData,
	FRepObjectDataBuffer OldData,
	FRepObjectDataBuffer Data,
	FGuidReferencesMap* GuidReferencesMap,
	bool& bOutHasUnmapped,
	bool& bOutGuidsChanged,
	UObject* OwningObject) const
{
	auto ReadHandle = [this, &Reader](uint32& Handle) -> bool
	{
		Reader.SerializeIntPacked(Handle);

		if (Reader.IsError())
		{
			UE_LOGF(LogRep, Warning, "ReceiveProperties_BackwardsCompatible_r: Error reading handle. Owner: %ls", *Owner->GetName());
			return false;
		}

		UE_LOGF(LogRepPropertiesBackCompat, VeryVerbose, "ReceiveProperties_BackwardsCompatible_r: NetFieldExportHandle=%d", Handle);
		return true;
	};

	if (NetFieldExportGroup == nullptr)
	{
		uint32 NetFieldExportHandle = 0;
		if (!ReadHandle(NetFieldExportHandle))
		{
			return false;
		}
		else if (NetFieldExportHandle != 0)
		{
			UE_CLOGF(!FApp::IsUnattended(), LogRep, Warning, "ReceiveProperties_BackwardsCompatible_r: NetFieldExportGroup == nullptr. Owner: %ls, NetFieldExportHandle: %u", *Owner->GetName(), NetFieldExportHandle);
			Reader.SetError();
			ensure(false);
			return false;
		}
		else
		{
			return true;
		}
	}

	FNetBitReader TempReader;
	TempReader.PackageMap = Reader.PackageMap;

	while (true)
	{
		uint32 NetFieldExportHandle = 0;
		if (!ReadHandle(NetFieldExportHandle))
		{
			return false;
		}

		if (NetFieldExportHandle == 0)
		{
			// We're done
			break;
		}

		// We purposely add 1 on save, so we can reserve 0 for "done"
		NetFieldExportHandle--;

		if (!ensure(NetFieldExportHandle < (uint32)NetFieldExportGroup->NetFieldExports.Num()))
		{
			UE_LOGF(LogRep, Warning, "ReceiveProperties_BackwardsCompatible_r: NetFieldExportHandle> NetFieldExportGroup->NetFieldExports.Num(). Owner: %ls, NetFieldExportHandle: %u", *Owner->GetName(), NetFieldExportHandle);
			return false;
		}

		const uint32 Checksum = NetFieldExportGroup->NetFieldExports[NetFieldExportHandle].CompatibleChecksum;

		if (!ensure(Checksum != 0))
		{
			UE_LOGF(LogRep, Warning, "ReceiveProperties_BackwardsCompatible_r: Checksum == 0. Owner: %ls, Name: %ls, NetFieldExportHandle: %i", *Owner->GetName(), *NetFieldExportGroup->NetFieldExports[NetFieldExportHandle].ExportName.ToString(), NetFieldExportHandle);
			return false;
		}

		uint32 NumBits = 0;
		Reader.SerializeIntPacked(NumBits);

		UE_LOGF(LogRepPropertiesBackCompat, VeryVerbose, "ReceiveProperties_BackwardsCompatible_r: NumBits=%d", NumBits);

		if (Reader.IsError())
		{
			UE_LOGF(LogRep, Warning, "ReceiveProperties_BackwardsCompatible_r: Error reading num bits. Owner: %ls, Name: %ls, NetFieldExportHandle: %i, Checksum: %u", *Owner->GetName(), *NetFieldExportGroup->NetFieldExports[NetFieldExportHandle].ExportName.ToString(), NetFieldExportHandle, Checksum);
			return false;
		}

		TempReader.ResetData(Reader, NumBits);

		if (Reader.IsError())
		{
			UE_LOGF(LogRep, Warning, "ReceiveProperties_BackwardsCompatible_r: Error reading payload. Owner: %ls, Name: %ls, NetFieldExportHandle: %i, Checksum: %u", *Owner->GetName(), *NetFieldExportGroup->NetFieldExports[NetFieldExportHandle].ExportName.ToString(), NetFieldExportHandle, Checksum);
			return false;
		}

		if (NetFieldExportGroup->NetFieldExports[NetFieldExportHandle].bIncompatible)
		{
			continue;		// We've already warned that this property doesn't load anymore
		}

		// Find this property
		const int32 CmdIndex = FindCompatibleProperty(CmdStart, CmdEnd, Checksum);

		if (CmdIndex == -1)
		{
			UE_LOGF(LogRep, Warning, "ReceiveProperties_BackwardsCompatible_r: Property not found. Owner: %ls, Name: %ls, NetFieldExportHandle: %i, Checksum: %u", *Owner->GetName(), *NetFieldExportGroup->NetFieldExports[NetFieldExportHandle].ExportName.ToString(), NetFieldExportHandle, Checksum);

			// Mark this property as incompatible so we don't keep spamming this warning
			NetFieldExportGroup->NetFieldExports[NetFieldExportHandle].bIncompatible = true;
			continue;
		}

		const FRepLayoutCmd& Cmd = Cmds[CmdIndex];

		if (Cmd.Type == ERepLayoutCmdType::DynamicArray)
		{
			uint32 ArrayNum = 0;
			TempReader.SerializeIntPacked(ArrayNum);

			UE_LOGF(LogRepPropertiesBackCompat, VeryVerbose, "ReceiveProperties_BackwardsCompatible_r: ArrayNum=%d", ArrayNum);

			if (TempReader.IsError())
			{
				UE_LOGF(LogRep, Warning, "ReceiveProperties_BackwardsCompatible_r: Error reading ArrayNum. Owner: %ls, Name: %ls, NetFieldExportHandle: %i, Checksum: %u", *Owner->GetName(), *NetFieldExportGroup->NetFieldExports[NetFieldExportHandle].ExportName.ToString(), NetFieldExportHandle, Checksum);
				return false;
			}

			if (ArrayNum > MAX_uint16)
			{
				UE_LOGF(LogRep, Warning, "ReceiveProperties_BackwardsCompatible_r: ArrayNum out of valid range [%u]. Owner: %ls, Name: %ls, NetFieldExportHandle: %i, Checksum: %u", ArrayNum, *Owner->GetName(), *NetFieldExportGroup->NetFieldExports[NetFieldExportHandle].ExportName.ToString(), NetFieldExportHandle, Checksum);
				return false;
			}

			const int32 AbsOffset = (Data.Data - OldData.Data) + Cmd.Offset;

			FScriptArray* DataArray = (FScriptArray*)(Data + Cmd).Data;
			FScriptArray* ShadowArray = ShadowData ? (FScriptArray*)(ShadowData + Cmd).Data : nullptr;

			FRepObjectDataBuffer LocalData = Data;
			FRepShadowDataBuffer LocalShadowData = ShadowData;

			bool bShadowDataCopied = false;
			TArray<FProperty*>* RepNotifies = (ShadowData && RepState) ? &RepState->RepNotifies : nullptr;

			FGuidReferencesMap* NewGuidReferencesArray = PrepReceivedArray(
				ArrayNum,
				ShadowArray,
				DataArray,
				GuidReferencesMap,
				AbsOffset,
				Parents[Cmd.ParentIndex],
				Cmd,
				CmdIndex,
				&LocalShadowData,
				&LocalData,
				RepNotifies,
				bShadowDataCopied,
				TempReader.PackageMap);

			// Read until we read all array elements
			while (true)
			{
				uint32 Index = 0;
				TempReader.SerializeIntPacked(Index);

				UE_LOGF(LogRepPropertiesBackCompat, VeryVerbose, "ReceiveProperties_BackwardsCompatible_r: ArrayIndex=%d", Index);

				if (TempReader.IsError())
				{
					UE_LOGF(LogRep, Warning, "ReceiveProperties_BackwardsCompatible_r: Error reading array index. Index: %i, Owner: %ls, Name: %ls, NetFieldExportHandle: %i, Checksum: %u", Index, *Owner->GetName(), *NetFieldExportGroup->NetFieldExports[NetFieldExportHandle].ExportName.ToString(), NetFieldExportHandle, Checksum);
					return false;
				}

				if (Index == 0)
				{
					// At this point, the 0 either signifies:
					//	An array terminator, at which point we're done.
					//	An array element terminator, which could happen if the array had tailing elements removed.
					if (TempReader.GetBitsLeft() == 8)
					{
						// We have bits left over, so see if its the Array Terminator.
						// This should be 0
						uint32 Terminator;
						TempReader.SerializeIntPacked(Terminator);

						if (Terminator != 0)
						{
							UE_LOGF(LogRep, Warning, "ReceiveProperties_BackwardsCompatible_r: Invalid array terminator. Owner: %ls, Name: %ls, NetFieldExportHandle: %i, Terminator: %d", *Owner->GetName(), *NetFieldExportGroup->NetFieldExports[NetFieldExportHandle].ExportName.ToString(), NetFieldExportHandle, Terminator);
							return false;
						}
					}

					// We're done
					break;
				}

				// Shift all indexes down since 0 represents null handle
				Index--;

				if (!ensure(Index < ArrayNum))
				{
					UE_LOGF(LogRep, Warning, "ReceiveProperties_BackwardsCompatible_r: Array index out of bounds. Index: %i, ArrayNum: %i, Owner: %ls, Name: %ls, NetFieldExportHandle: %i, Checksum: %u", Index, ArrayNum, *Owner->GetName(), *NetFieldExportGroup->NetFieldExports[NetFieldExportHandle].ExportName.ToString(), NetFieldExportHandle, Checksum);
					return false;
				}

				const int32 ArrayElementOffset = Index * Cmd.ElementSize;

				FRepObjectDataBuffer ElementData = LocalData + ArrayElementOffset;
				FRepShadowDataBuffer ElementShadowData = (LocalShadowData && (Index < (ShadowArray ? static_cast<uint32>(ShadowArray->Num()) : 0))) ? LocalShadowData + ArrayElementOffset : nullptr;

				if (!ReceiveProperties_BackwardsCompatible_r(RepState, NetFieldExportGroup, TempReader, CmdIndex + 1, Cmd.EndCmd - 1, ElementShadowData, LocalData, ElementData, NewGuidReferencesArray, bOutHasUnmapped, bOutGuidsChanged, OwningObject))
				{
					return false;
				}

				if (TempReader.IsError())
				{
					UE_LOGF(LogRep, Warning, "ReceiveProperties_BackwardsCompatible_r: Error reading array index element payload. Index: %i, Owner: %ls, Name: %ls, NetFieldExportHandle: %i, Checksum: %u", Index, *Owner->GetName(), *NetFieldExportGroup->NetFieldExports[NetFieldExportHandle].ExportName.ToString(), NetFieldExportHandle, Checksum);
					return false;
				}
			}

			if (TempReader.GetBitsLeft() != 0)
			{
				UE_LOG(LogRep, Warning, TEXT("ReceiveProperties_BackwardsCompatible_r: Array didn't read proper number of bits. Owner: %s, Name: %s, NetFieldExportHandle: %i, Checksum: %u, BitsLeft:%" INT64_FMT), *Owner->GetName(), *NetFieldExportGroup->NetFieldExports[NetFieldExportHandle].ExportName.ToString(), NetFieldExportHandle, Checksum, TempReader.GetBitsLeft());
				return false;
			}
		}
		else
		{
			const int32 ElementOffset = (Data.Data - OldData.Data);

			TArray<FProperty*>* RepNotifies = (ShadowData && RepState) ? &RepState->RepNotifies : nullptr;
			if (ReceivePropertyHelper(
				TempReader,
				GuidReferencesMap,
				ElementOffset,
				ShadowData,
				Data,
				RepNotifies,
				false /*bShadowDataCopied*/,
				Parents,
				Cmds,
				CmdIndex,
				false,
				bOutGuidsChanged,

				// We can skip role swapping if we're not an actor.
				!EnumHasAnyFlags(Flags, ERepLayoutFlags::IsActor),
				NetSerializeLayouts,
				OwningObject))
			{
				bOutHasUnmapped = true;
			}

			if (TempReader.GetBitsLeft() != 0)
			{
				UE_LOG(LogRep, Warning, TEXT("ReceiveProperties_BackwardsCompatible_r: Property didn't read proper number of bits. Owner: %s, Name: %s, NetFieldExportHandle: %i, Checksum: %u, BitsLeft:%" INT64_FMT), *Owner->GetName(), *NetFieldExportGroup->NetFieldExports[NetFieldExportHandle].ExportName.ToString(), NetFieldExportHandle, Checksum, TempReader.GetBitsLeft());
				return false;
			}

			if (TempReader.IsError())
			{
				UE_LOGF(LogRep, Warning, "ReceiveProperties_BackwardsCompatible_r: Error Reading Property. Owner: %ls, Name: %ls, NetFieldExportHandle: %i, Checksum: %u", *Owner->GetName(), *NetFieldExportGroup->NetFieldExports[NetFieldExportHandle].ExportName.ToString(), NetFieldExportHandle, Checksum);
				return false;
			}
		}
	}

	return true;
}

void FRepLayout::GatherGuidReferences_r(
	const FGuidReferencesMap* GuidReferencesMap,
	TSet<FNetworkGUID>& OutReferencedGuids,
	int32& OutTrackedGuidMemoryBytes) const
{
	for (const auto& GuidReferencePair : *GuidReferencesMap)
	{
		const FGuidReferences& GuidReferences = GuidReferencePair.Value;

		if (GuidReferences.Array != NULL)
		{
			check(Cmds[GuidReferences.CmdIndex].Type == ERepLayoutCmdType::DynamicArray);

			GatherGuidReferences_r(GuidReferences.Array, OutReferencedGuids, OutTrackedGuidMemoryBytes);
			continue;
		}

		OutTrackedGuidMemoryBytes += GuidReferences.Buffer.Num();

		OutReferencedGuids.Append(GuidReferences.GetUnmappedGUIDs());
		OutReferencedGuids.Append(GuidReferences.MappedDynamicGUIDs);
	}
}

void FRepLayout::GatherGuidReferences(
	FReceivingRepState* RESTRICT RepState,
	FNetDeltaSerializeInfo& Params,
	TSet<FNetworkGUID>& OutReferencedGuids,
	int32& OutTrackedGuidMemoryBytes) const
{
	if (!IsEmpty())
	{
		GatherGuidReferences_r(&RepState->GuidReferencesMap, OutReferencedGuids, OutTrackedGuidMemoryBytes);

		// Custom Delta Properties
		if (LifetimeCustomPropertyState)
		{
			FRepObjectDataBuffer ObjectData(Params.Object);
			const int32 NumLifetimeCustomDeltaProperties = LifetimeCustomPropertyState->GetNumCustomDeltaProperties();

			for (int32 CustomDeltaIndex = 0; CustomDeltaIndex < NumLifetimeCustomDeltaProperties; ++CustomDeltaIndex)
			{
				const FLifetimeCustomDeltaProperty& CustomDeltaProperty = LifetimeCustomPropertyState->GetCustomDeltaProperty(CustomDeltaIndex);
				const FRepParentCmd& Parent = Parents[CustomDeltaProperty.PropertyRepIndex];

				// Static cast is safe here, because this property wouldn't have been marked CustomDelta otherwise.
				FStructProperty* StructProperty = static_cast<FStructProperty*>(Parent.Property);
				UScriptStruct::ICppStructOps* CppStructOps = StructProperty->Struct->GetCppStructOps();

				FNetDeltaSerializeInfo TempParams = Params;
				TempParams.Struct = StructProperty->Struct;
				TempParams.CustomDeltaIndex = CustomDeltaIndex;
				TempParams.Data = ObjectData + Parent;

				CppStructOps->NetDeltaSerialize(TempParams, TempParams.Data);
			}
		}
	}
}

bool FRepLayout::MoveMappedObjectToUnmapped_r(FGuidReferencesMap* GuidReferencesMap, const FNetworkGUID& GUID, const UObject* const OwningObject) const
{
	bool bFoundGUID = false;

	for (auto& GuidReferencePair : *GuidReferencesMap)
	{
		FGuidReferences& GuidReferences = GuidReferencePair.Value;

		if (GuidReferences.Array != NULL)
		{
			check(Cmds[GuidReferences.CmdIndex].Type == ERepLayoutCmdType::DynamicArray);

			if (MoveMappedObjectToUnmapped_r(GuidReferences.Array, GUID, OwningObject))
			{
				bFoundGUID = true;
			}
			continue;
		}

		if (GuidReferences.MappedDynamicGUIDs.Contains(GUID))
		{
			GuidReferences.MappedDynamicGUIDs.Remove(GUID);
			GuidReferences.AddUnmappedGUID(GUID);
			bFoundGUID = true;

#if WITH_PUSH_MODEL
			if (OwningObject)
			{
				FNetPrivatePushIdHelper::MarkPropertyDirty(OwningObject, GuidReferences.ParentIndex);
			}
#endif
		}
	}

	return bFoundGUID;
}

bool FRepLayout::MoveMappedObjectToUnmapped(
	FReceivingRepState* RESTRICT RepState,
	FNetDeltaSerializeInfo& Params,
	const FNetworkGUID& GUID) const
{
	bool bFound = false;

	if (!IsEmpty())
	{
		bFound = MoveMappedObjectToUnmapped_r(&RepState->GuidReferencesMap, GUID, Params.Object);

		// Custom Delta Properties
		if (LifetimeCustomPropertyState && Params.Object)
		{
			FRepObjectDataBuffer ObjectData(Params.Object);
			const int32 NumLifetimeCustomDeltaProperties = LifetimeCustomPropertyState->GetNumCustomDeltaProperties();

			for (int32 CustomDeltaIndex = 0; CustomDeltaIndex < NumLifetimeCustomDeltaProperties; ++CustomDeltaIndex)
			{
				const FLifetimeCustomDeltaProperty& CustomDeltaProperty = LifetimeCustomPropertyState->GetCustomDeltaProperty(CustomDeltaIndex);
				const FRepParentCmd& Parent = Parents[CustomDeltaProperty.PropertyRepIndex];

				// Static cast is safe here, because this property wouldn't have been marked CustomDelta otherwise.
				FStructProperty* StructProperty = static_cast<FStructProperty*>(Parent.Property);
				UScriptStruct::ICppStructOps* CppStructOps = StructProperty->Struct->GetCppStructOps();

				FNetDeltaSerializeInfo TempParams = Params;

				TempParams.Struct = StructProperty->Struct;
				TempParams.Data = ObjectData + Parent;
				TempParams.CustomDeltaIndex = CustomDeltaIndex;
				TempParams.bOutHasMoreUnmapped = false;
				TempParams.bOutSomeObjectsWereMapped = false;

				if (CppStructOps->NetDeltaSerialize(TempParams, TempParams.Data))
				{
					bFound = true;
				}

				Params.bOutHasMoreUnmapped |= TempParams.bOutHasMoreUnmapped;
				Params.bOutSomeObjectsWereMapped |= TempParams.bOutSomeObjectsWereMapped;
			}
		}
	}

	return bFound;
}

void FRepLayout::UpdateUnmappedObjects_r(
	FReceivingRepState* RESTRICT RepState,
	FGuidReferencesMap* GuidReferencesMap,
	UObject* OriginalObject,
	UNetConnection* Connection,
	FRepShadowDataBuffer ShadowData,
	FRepObjectDataBuffer Data,
	const int32 MaxAbsOffset,
	bool& bCalledPreNetReceive,
	bool& bOutSomeObjectsWereMapped,
	bool& bOutHasMoreUnmapped) const
{
	for (auto It = GuidReferencesMap->CreateIterator(); It; ++It)
	{
		const int32 AbsOffset = It.Key();

		if (AbsOffset >= MaxAbsOffset)
		{
			// Array must have shrunk, we can remove this item
			UE_LOGF(LogRep, VeryVerbose, "UpdateUnmappedObjects_r: REMOVED unmapped property: AbsOffset>= MaxAbsOffset. Offset: %i", AbsOffset);
			It.RemoveCurrent();
			continue;
		}

		FGuidReferences& GuidReferences = It.Value();
		const FRepLayoutCmd& Cmd = Cmds[GuidReferences.CmdIndex];
		const FRepParentCmd& Parent = Parents[GuidReferences.ParentIndex];
		const bool bUpdateShadowState = (ShadowData && INDEX_NONE != Parent.RepNotifyNumParams);

		// Make sure if we're touching an array element, we use the correct offset for shadow values.
		// This should always be safe, because MaxAbsOffset will account for ShadowArray size for arrays.
		// For non array properties, AbsOffset should always equal Cmd.Offset.
		const int32 ShadowOffset = (AbsOffset - Cmd.Offset) + Cmd.ShadowOffset;

		if (GuidReferences.Array != nullptr)
		{
			check(Cmd.Type == ERepLayoutCmdType::DynamicArray);

			if (bUpdateShadowState)
			{
				FScriptArray* ShadowArray = (FScriptArray*)(ShadowData + ShadowOffset).Data;
				FScriptArray* Array = (FScriptArray*)(Data + AbsOffset).Data;

				FRepShadowDataBuffer ShadowArrayData(ShadowArray->GetData());
				FRepObjectDataBuffer ArrayData(Array->GetData());

				const int32 NewMaxOffset = FMath::Min(ShadowArray->Num() * Cmd.ElementSize, Array->Num() * Cmd.ElementSize);

				UpdateUnmappedObjects_r(RepState, GuidReferences.Array, OriginalObject, Connection, ShadowArrayData, ArrayData, NewMaxOffset, bCalledPreNetReceive, bOutSomeObjectsWereMapped, bOutHasMoreUnmapped);
			}
			else
			{
				FScriptArray* Array = (FScriptArray*)(Data + AbsOffset).Data;
				FRepObjectDataBuffer ArrayData(Array->GetData());
				const int32 NewMaxOffset = Array->Num() * Cmd.ElementSize;

				UpdateUnmappedObjects_r(RepState, GuidReferences.Array, OriginalObject, Connection, nullptr, ArrayData, NewMaxOffset, bCalledPreNetReceive, bOutSomeObjectsWereMapped, bOutHasMoreUnmapped);
			}
			continue;
		}

		bool bMappedSomeGUIDs = GuidReferences.UpdateUnmappedGUIDs(Connection->PackageMap, OriginalObject, Cmd.Property, AbsOffset);

		// If we resolved some guids, re-deserialize the data which will hook up the object pointer with the property
		if (bMappedSomeGUIDs)
		{
			if (!bCalledPreNetReceive)
			{
				// Call PreNetReceive if we are going to change a value (some game code will need to think this is an actual replicated value)
				OriginalObject->PreNetReceive();
				bCalledPreNetReceive = true;
			}

			bOutSomeObjectsWereMapped = true;

			// Copy current value over so we can check to see if it changed
			if (bUpdateShadowState)
			{
				StoreProperty(Cmd, ShadowData + ShadowOffset, Data + AbsOffset);
			}

			// Initialize the reader with the stored buffer that we need to read from
			FNetBitReader Reader(Connection->PackageMap, GuidReferences.Buffer.GetData(), GuidReferences.NumBufferBits);
			Connection->SetNetVersionsOnArchive(Reader);

			// Read the property
			Cmd.Property->NetSerializeItem(Reader, Connection->PackageMap, Data + AbsOffset);

			// Check to see if this property changed
			if (bUpdateShadowState && RepState)
			{
				// I have a sneaking suspicion that this is broken.
				// AbsOffset could be Cmd.Offset, but we also may be recursing into an Array, and that
				// would mean it could be Cmd.Offset + (ArrayIndex * ElementOffset)
				// That could cause us to trigger RepNotifies more often for Dynamic Array properties.
				// That goes for the above too.

				if (Parent.RepNotifyCondition == REPNOTIFY_Always || !PropertiesAreIdentical(Cmd, ShadowData + ShadowOffset, Data + AbsOffset, NetSerializeLayouts))
				{
					// If this properties needs an OnRep, queue that up to be handled later
					RepState->RepNotifies.AddUnique(Parent.Property);
				}
				else
				{
					UE_CLOGF(LogSkippedRepNotifies, LogRep, Display, "UpdateUnmappedObjects_r: Skipping RepNotify because Property did not change. %ls", *Cmd.Property->GetName());
				}
			}
		}

		// If we still have more unmapped guids, we need to keep processing this entry
		if (GuidReferences.GetUnmappedGUIDs().Num() > 0)
		{
			bOutHasMoreUnmapped = true;
		}
		else if (GuidReferences.GetUnmappedGUIDs().Num() == 0 && GuidReferences.MappedDynamicGUIDs.Num() == 0)
		{
			It.RemoveCurrent();
		}
	}
}

void FRepLayout::UpdateUnmappedObjects(
	FReceivingRepState* RESTRICT RepState,
	UPackageMap* PackageMap,
	UObject* OriginalObject,
	FNetDeltaSerializeInfo& Params,
	bool& bCalledPreNetReceive,
	bool& bOutSomeObjectsWereMapped,
	bool& bOutHasMoreUnmapped) const
{
	bOutSomeObjectsWereMapped = false;
	bOutHasMoreUnmapped = false;
	bCalledPreNetReceive = false;

	if (!IsEmpty())
	{
		UpdateUnmappedObjects_r(
			RepState,
			&RepState->GuidReferencesMap,
			OriginalObject,
			Params.Connection,
			(uint8*)RepState->StaticBuffer.GetData(),
			(uint8*)OriginalObject,
			Owner->GetPropertiesSize(),
			bCalledPreNetReceive,
			bOutSomeObjectsWereMapped,
			bOutHasMoreUnmapped);

		Params.bCalledPreNetReceive = bCalledPreNetReceive;

		// Custom Delta Properties
		if (LifetimeCustomPropertyState)
		{
			FRepObjectDataBuffer ObjectData(Params.Object);
			const int32 NumLifetimeCustomDeltaProperties = LifetimeCustomPropertyState->GetNumCustomDeltaProperties();

			for (int32 CustomDeltaIndex = 0; CustomDeltaIndex < NumLifetimeCustomDeltaProperties; ++CustomDeltaIndex)
			{
				const FLifetimeCustomDeltaProperty& CustomDeltaProperty = LifetimeCustomPropertyState->GetCustomDeltaProperty(CustomDeltaIndex);
				const FRepParentCmd& Parent = Parents[CustomDeltaProperty.PropertyRepIndex];

				// Static cast is safe here, because this property wouldn't have been marked CustomDelta otherwise.
				FStructProperty* StructProperty = static_cast<FStructProperty*>(Parent.Property);
				UScriptStruct::ICppStructOps* CppStructOps = StructProperty->Struct->GetCppStructOps();

				FNetDeltaSerializeInfo TempParams = Params;

				TempParams.DebugName = Parent.CachedPropertyName.ToString();
				TempParams.Struct = StructProperty->Struct;
				TempParams.bOutSomeObjectsWereMapped = false;
				TempParams.bOutHasMoreUnmapped = false;
				TempParams.CustomDeltaIndex = CustomDeltaIndex;
				TempParams.Data = ObjectData + Parent;

				// Call the custom delta serialize function to handle it
				CppStructOps->NetDeltaSerialize(TempParams, TempParams.Data);

				if (TempParams.bOutSomeObjectsWereMapped && INDEX_NONE != Parent.RepNotifyNumParams)
				{
					UE_RepLayout_Private::QueueRepNotifyForCustomDeltaProperty(RepState, Params, StructProperty, Parent.ArrayIndex);
				}

				Params.bOutSomeObjectsWereMapped |= TempParams.bOutSomeObjectsWereMapped;
				Params.bOutHasMoreUnmapped |= TempParams.bOutHasMoreUnmapped;
				Params.bCalledPreNetReceive |= TempParams.bCalledPreNetReceive;
			}
		}
	}
}

bool FRepLayout::SendCustomDeltaProperty(FNetDeltaSerializeInfo& Params, uint16 CustomDeltaIndex) const
{
	const FLifetimeCustomDeltaProperty& CustomDeltaProperty = LifetimeCustomPropertyState->GetCustomDeltaProperty(CustomDeltaIndex);
	const FRepParentCmd& Parent = Parents[CustomDeltaProperty.PropertyRepIndex];

	if (!ensure(EnumHasAnyFlags(Parent.Flags, ERepParentFlags::IsCustomDelta)))
	{
		return false;
	}

	FStructProperty* StructProperty = static_cast<FStructProperty*>(Parent.Property);
	UScriptStruct::ICppStructOps * CppStructOps = StructProperty->Struct->GetCppStructOps();

	check(CppStructOps); // else should not have STRUCT_NetSerializeNative

	Params.DebugName = Parent.CachedPropertyName.ToString();
	Params.Struct = StructProperty->Struct;
	Params.CustomDeltaIndex = CustomDeltaIndex;
	Params.Data = FRepObjectDataBuffer(Params.Object) + Parent;

	bool bSupportsFastArrayDelta = Params.bSupportsFastArrayDeltaStructSerialization;

	if (Params.bSupportsFastArrayDeltaStructSerialization &&
		EnumHasAnyFlags(Parent.Flags, ERepParentFlags::IsFastArray) &&
		!!LifetimeCustomPropertyState->GetNumFastArrayProperties())
	{
		bSupportsFastArrayDelta = CustomDeltaProperty.FastArrayNumber != INDEX_NONE;
	}

	Params.Writer->WriteBit(bSupportsFastArrayDelta);

	if (Parent.Property->ArrayDim != 1)
	{
		uint32 StaticArrayIndex = Parent.ArrayIndex;
		Params.Writer->SerializeIntPacked(StaticArrayIndex);
	}

#if WITH_PUSH_MODEL
	if (EnumHasAnyFlags(Parent.Flags, ERepParentFlags::IsFastArray) && IsPushModelProperty(CustomDeltaProperty.PropertyRepIndex) && Params.CustomDeltaObject)
	{
		void* ArrayData = FRepObjectDataBuffer(Params.CustomDeltaObject) + Parent;
		static_cast<FFastArraySerializer*>(ArrayData)->CachePushModelState(Params.CustomDeltaObject, CustomDeltaProperty.PropertyRepIndex);
	}
#endif

	TGuardValue<bool> SupportFastArrayDeltaGuard(Params.bSupportsFastArrayDeltaStructSerialization, bSupportsFastArrayDelta);
	return CppStructOps->NetDeltaSerialize(Params, Params.Data);
}

bool FRepLayout::ReceiveCustomDeltaProperty(
	FReceivingRepState* RESTRICT ReceivingRepState,
	FNetDeltaSerializeInfo& Params,
	FStructProperty* Property) const
{
	if (Params.Connection->GetNetworkCustomVersion(FEngineNetworkCustomVersion::Guid) >= FEngineNetworkCustomVersion::FastArrayDeltaStruct)
	{
		Params.bSupportsFastArrayDeltaStructSerialization = !!Params.Reader->ReadBit();
	}
	else
	{
		Params.bSupportsFastArrayDeltaStructSerialization = false;
	}

	uint32 StaticArrayIndex = 0;

	// Receive array index (static sized array, i.e. MemberVariable[4])
	if (Property->ArrayDim != 1)
	{
		check(Property->ArrayDim >= 2);

		Params.Reader->SerializeIntPacked(StaticArrayIndex);

		if (StaticArrayIndex >= (uint32)Property->ArrayDim)
		{
			UE_LOGF(LogRep, Error, "Element index too large %ls in %ls", *Property->GetName(), *Params.Object->GetFullName());
			return false;
		}
	}

	const FRepParentCmd& Parent = Parents[Property->RepIndex + StaticArrayIndex];

	// We should only be receiving custom delta properties (since RepLayout handles the rest)
	if (!EnumHasAnyFlags(Parent.Flags, ERepParentFlags::IsCustomDelta))
	{
		UE_LOGF(LogNet, Error, "Client received non custom delta property value %ls in %ls", *Parent.CachedPropertyName.ToString(), *Params.Object->GetFullName());
		return false;
	}


	UScriptStruct* InnerStruct = Property->Struct;
	UScriptStruct::ICppStructOps* CppStructOps = InnerStruct->GetCppStructOps();

	check(CppStructOps);

	Params.DebugName = Parent.CachedPropertyName.ToString();
	Params.Struct = InnerStruct;
	Params.CustomDeltaIndex = LifetimeCustomPropertyState->GetCustomDeltaIndexFromPropertyRepIndex(Property->RepIndex + StaticArrayIndex);
	Params.Data = FRepObjectDataBuffer(Params.Object) + Parent;

	if (CppStructOps->NetDeltaSerialize(Params, Params.Data))
	{
		if (UNLIKELY(Params.Reader->IsError()))
		{
			UE_LOGF(LogNet, Error, "FRepLayout::ReceiveCustomDeltaProperty: NetDeltaSerialize - Reader.IsError() == true. Property: %ls, Object: %ls", *Params.DebugName, *Params.Object->GetFullName());
			return false;
		}
		if (UNLIKELY(Params.Reader->GetBitsLeft() != 0))
		{
			UE_LOGF(LogNet, Error, "FRepLayout::ReceiveCustomDeltaProperty: NetDeltaSerialize - Mismatch read. Property: %ls, Object: %ls", *Params.DebugName, *Params.Object->GetFullName());
			return false;
		}

		// Successfully received it.
		if (INDEX_NONE != Parent.RepNotifyNumParams)
		{
			UE_RepLayout_Private::QueueRepNotifyForCustomDeltaProperty(ReceivingRepState, Params, Property, StaticArrayIndex);
		}

		return true;
	}

	return false;
}

void FRepLayout::CallRepNotifies(FReceivingRepState* RepState, UObject* Object) const
{
	if (RepState->RepNotifies.Num() == 0)
	{
		return;
	}
	
	if (IsEmpty())
	{
		UE_LOGF(LogRep, Error, "FRepLayout::CallRepNotifies: Empty layout with RepNotifies: %ls", *GetPathNameSafe(Owner));
		return;
	}

	CSV_SCOPED_TIMING_STAT_EXCLUSIVE(RepNotifies_Generic);

	FRepShadowDataBuffer ShadowData(RepState->StaticBuffer.GetData());
	FRepObjectDataBuffer ObjectData(Object);

	for (FProperty* RepProperty : RepState->RepNotifies)
	{
		if (!Parents.IsValidIndex(RepProperty->RepIndex))
		{
			UE_LOGF(LogRep, Warning, "FRepLayout::CallRepNotifies: Called with invalid property %ls on object %ls.",
				*RepProperty->GetName(), *Object->GetName());
				continue;
		}

		UFunction* RepNotifyFunc = Object->FindFunction(RepProperty->RepNotifyFunc);

		if (RepNotifyFunc == nullptr)
		{
			UE_LOGF(LogRep, Warning, "FRepLayout::CallRepNotifies: Can't find RepNotify function %ls for property %ls on object %ls.",
				*RepProperty->RepNotifyFunc.ToString(), *RepProperty->GetName(), *Object->GetName());
			continue;
		}

		const FRepParentCmd& Parent = Parents[RepProperty->RepIndex];
		const int32 NumParms = RepNotifyFunc->NumParms;

		switch (NumParms)
		{
			case 0:
			{
				Object->ProcessEvent(RepNotifyFunc, nullptr);

				// TODO CopyCompleteValue no matter the field is replicated or not
				// will be a performance regression for any RepNotify arrays.
				// One fix is to track the incoming changelist and then resize the array and 
				// recursively copy over only the fields we care about.
				if (EnumHasAnyFlags(Parent.Flags, ERepParentFlags::HasDynamicArrayProperties) && !EnumHasAnyFlags(Parent.Flags, ERepParentFlags::IsFastArray))
				{
					RepProperty->CopyCompleteValue(ShadowData + Parent, ObjectData + Parent);
				}
				break;
			}
			case 1:
			{
				FRepShadowDataBuffer PropertyData = ShadowData + Parent;

				if (EnumHasAnyFlags(Parent.Flags, ERepParentFlags::IsCustomDelta))
				{
					Object->ProcessEvent(RepNotifyFunc, PropertyData);
				}
				else
				{
					// Handle bitfields.
					const FBoolProperty* BoolProperty = CastField<const FBoolProperty>(Parent.Property);
					if (BoolProperty && !BoolProperty->IsNativeBool())
					{
						bool BoolPropertyValue = BoolProperty->GetPropertyValue(PropertyData);
						Object->ProcessEvent(RepNotifyFunc, &BoolPropertyValue);
					}
					else
					{
						Object->ProcessEvent(RepNotifyFunc, PropertyData);
					}

					// now store the complete value in the shadow buffer
					if (!EnumHasAnyFlags(Parent.Flags, ERepParentFlags::IsNetSerialize))
					{
						RepProperty->CopyCompleteValue(ShadowData + Parent, ObjectData + Parent);
					}
				}
				break;
			}
			case 2:
			{
				check(EnumHasAnyFlags(Parent.Flags, ERepParentFlags::IsCustomDelta));

				// Fixme: this isn't as safe as it could be. Right now we have two types of parameters: MetaData (a TArray<uint8>)
				// and the last local value (pointer into the Recent[] array).
				//
				// Arrays always expect MetaData. Everything else, including structs, expect last value.
				// This is enforced with UHT only. If a ::NetSerialize function ever starts producing a MetaData array thats not in FArrayProperty,
				// we have no static way of catching this and the replication system could pass the wrong thing into ProcessEvent here.
				//
				// But this is all sort of an edge case feature anyways, so its not worth tearing things up too much over.

				FMemMark Mark(FMemStack::Get());
				uint8* Parms = new(FMemStack::Get(), MEM_Zeroed, RepNotifyFunc->ParmsSize)uint8;

				TFieldIterator<FProperty> Itr(RepNotifyFunc);
				check(Itr);

				FRepShadowDataBuffer PropertyData = ShadowData + Parent;

				Itr->CopyCompleteValue(Itr->ContainerPtrToValuePtr<void>(Parms), PropertyData);
				++Itr;
				check(Itr);

				TArray<uint8> *NotifyMetaData = RepState->RepNotifyMetaData.Find(RepProperty);
				check(NotifyMetaData);
				Itr->CopyCompleteValue(Itr->ContainerPtrToValuePtr<void>(Parms), NotifyMetaData);

				Object->ProcessEvent(RepNotifyFunc, Parms);

				Mark.Pop();
				break;
			}
			default:
			{
				checkf(false, TEXT("FRepLayout::CallRepNotifies: Invalid number of parameters for property %s on object %s. NumParms=%d, CustomDelta=%d"),
					*RepProperty->GetName(), *Object->GetName(), NumParms, !!EnumHasAnyFlags(Parent.Flags, ERepParentFlags::IsCustomDelta));
				break;
			}
		}
	}

	RepState->RepNotifies.Empty();
	RepState->RepNotifyMetaData.Empty();
}

template<ERepDataBufferType DataType>
static void ValidateWithChecksum_r(
	TArray<FRepLayoutCmd>::TConstIterator& CmdIt,
	const TConstRepDataBuffer<DataType> Data,
	FBitArchive& Ar);

template<ERepDataBufferType DataType>
static void ValidateWithChecksum_DynamicArray_r(
	TArray<FRepLayoutCmd>::TConstIterator& CmdIt,
	TConstRepDataBuffer<DataType> Data,
	FBitArchive& Ar)
{
	const FRepLayoutCmd& Cmd = *CmdIt;
	if (EnumHasAllFlags(Cmd.Flags, ERepLayoutCmdFlags::IsEmptyArrayStruct))
	{
		return;
	}

	// -2 because the current index will be the Owner Array Properties Cmd Index (+1)
	// and EndCmd will be the Cmd Index just *after* the Return Command (+1) 
	const int32 ArraySubCommands = CmdIt.GetIndex() - Cmd.EndCmd - 2;

	FScriptArray* Array = (FScriptArray*)Data.Data;

	uint16 ArrayNum = Array->Num();
	uint16 ElementSize = Cmd.ElementSize;

	Ar << ArrayNum;
	Ar << ElementSize;

	if (ArrayNum != Array->Num())
	{
		UE_LOGF(LogRep, Fatal, "ValidateWithChecksum_AnyArray_r: Array sizes different! %ls %i / %i", *Cmd.Property->GetFullName(), ArrayNum, Array->Num());
	}

	if (ElementSize != Cmd.ElementSize)
	{
		UE_LOGF(LogRep, Fatal, "ValidateWithChecksum_AnyArray_r: Array element sizes different! %ls %i / %i", *Cmd.Property->GetFullName(), ElementSize, Cmd.ElementSize);
	}

	const TConstRepDataBuffer<DataType> ArrayData(Array->GetData());
	for (int32 i = 0; i < ArrayNum - 1; i++)
	{
		const int32 ArrayElementsOffset = i * ElementSize;
		ValidateWithChecksum_r<>(CmdIt, Data + ArrayElementsOffset, Ar);
		CmdIt -= ArraySubCommands;
	}

	const int32 ArrayElementOffset = (ArrayNum - 1) * ElementSize;
	ValidateWithChecksum_r<>(CmdIt, ArrayData + ArrayElementOffset, Ar);
}

template<ERepDataBufferType DataType>
void ValidateWithChecksum_r(
	TArray<FRepLayoutCmd>::TConstIterator& CmdIt,
	const TConstRepDataBuffer<DataType> Data, 
	FBitArchive& Ar)
{
	for (; CmdIt->Type != ERepLayoutCmdType::Return; ++CmdIt)
	{
		const FRepLayoutCmd& Cmd = *CmdIt;
		if (Cmd.Type == ERepLayoutCmdType::DynamicArray)
		{
			ValidateWithChecksum_DynamicArray_r<>(CmdIt, Data + Cmd, Ar);
		}
		else
		{
			SerializeReadWritePropertyChecksum<>(Cmd, CmdIt.GetIndex() - 1, Data + Cmd, Ar);
		}
	}
}

template<ERepDataBufferType DataType>
void FRepLayout::ValidateWithChecksum(TConstRepDataBuffer<DataType> Data, FBitArchive& Ar) const
{
	TArray<FRepLayoutCmd>::TConstIterator CmdIt = Cmds.CreateConstIterator();
	ValidateWithChecksum_r<>(CmdIt, Data, Ar);
	check(CmdIt.GetIndex() == Cmds.Num());
}

uint32 FRepLayout::GenerateChecksum(const FRepState* RepState) const
{
	FBitWriter Writer(1024, true);
	ValidateWithChecksum<>(FConstRepShadowDataBuffer(RepState->GetReceivingRepState()->StaticBuffer.GetData()), Writer);

	return FCrc::MemCrc32(Writer.GetData(), Writer.GetNumBytes(), 0);
}

void FRepLayout::PruneChangeList(
	const FConstRepObjectDataBuffer Data,
	const TArray<uint16>& Changed,
	TArray<uint16>& PrunedChanged) const
{
	check(Changed.Num() > 0);

	PrunedChanged.Empty(1);

	if (!IsEmpty())
	{
		FChangelistIterator ChangelistIterator(Changed, 0);
		FRepHandleIterator HandleIterator(Owner, ChangelistIterator, Cmds, BaseHandleToCmdIndex, 0, 1, 0, Cmds.Num() - 1);
		PruneChangeList_r(HandleIterator, Data, PrunedChanged);
	}

	PrunedChanged.Add(0);
}

void FRepLayout::MergeChangeList(
	const FConstRepObjectDataBuffer Data,
	const TArray<uint16>& Dirty1,
	const TArray<uint16>& Dirty2,
	TArray<uint16>& MergedDirty) const
{
	check(Dirty1.Num() > 0);
	MergedDirty.Empty(1);

	if (!IsEmpty())
	{
		if (Dirty2.Num() == 0)
		{
			FChangelistIterator ChangelistIterator(Dirty1, 0);
			FRepHandleIterator HandleIterator(Owner, ChangelistIterator, Cmds, BaseHandleToCmdIndex, 0, 1, 0, Cmds.Num() - 1);
			PruneChangeList_r(HandleIterator, Data, MergedDirty);
		}
		else
		{
			FChangelistIterator ChangelistIterator1(Dirty1, 0);
			FRepHandleIterator HandleIterator1(Owner, ChangelistIterator1, Cmds, BaseHandleToCmdIndex, 0, 1, 0, Cmds.Num() - 1);

			FChangelistIterator ChangelistIterator2(Dirty2, 0);
			FRepHandleIterator HandleIterator2(Owner, ChangelistIterator2, Cmds, BaseHandleToCmdIndex, 0, 1, 0, Cmds.Num() - 1);

			MergeChangeList_r(HandleIterator1, HandleIterator2, Data, MergedDirty);
		}
	}

	MergedDirty.Add(0);
}

void FRepLayout::SanityCheckChangeList_DynamicArray_r(
	const int32 CmdIndex, 
	const FConstRepObjectDataBuffer Data, 
	TArray<uint16>& Changed,
	int32& ChangedIndex) const
{
	const FRepLayoutCmd& Cmd = Cmds[CmdIndex];

	FScriptArray * Array = (FScriptArray *)Data.Data;

	// Read the jump offset
	// We won't need to actually jump over anything because we expect the change list to be pruned once we get here
	// But we can use it to verify we read the correct amount.
	const int32 ArrayChangedCount = Changed[ChangedIndex++];

	const int32 OldChangedIndex = ChangedIndex;

	const FConstRepObjectDataBuffer ArrayData = (uint8*)Array->GetData();

	uint16 LocalHandle = 0;

	for (int32 i = 0; i < Array->Num(); i++)
	{
		const int32 ArrayElementOffset = i * Cmd.ElementSize;
		LocalHandle = SanityCheckChangeList_r(CmdIndex + 1, Cmd.EndCmd - 1, ArrayData + ArrayElementOffset, Changed, ChangedIndex, LocalHandle);
	}

	check(ChangedIndex - OldChangedIndex == ArrayChangedCount);	// Make sure we read correct amount
	check(Changed[ChangedIndex] == 0);							// Make sure we are at the end

	ChangedIndex++;
}

uint16 FRepLayout::SanityCheckChangeList_r(
	const int32 CmdStart, 
	const int32 CmdEnd, 
	const FConstRepObjectDataBuffer Data, 
	TArray<uint16>& Changed,
	int32& ChangedIndex,
	uint16 Handle 
	) const
{
	for (int32 CmdIndex = CmdStart; CmdIndex < CmdEnd; CmdIndex++)
	{
		const FRepLayoutCmd& Cmd = Cmds[ CmdIndex ];

		check(Cmd.Type != ERepLayoutCmdType::Return);

		Handle++;

		if (Cmd.Type == ERepLayoutCmdType::DynamicArray)
		{
			if (Handle == Changed[ChangedIndex])
			{
				const int32 LastChangedArrayHandle = Changed[ChangedIndex];
				ChangedIndex++;
				SanityCheckChangeList_DynamicArray_r(CmdIndex, Data + Cmd, Changed, ChangedIndex);
				check(Changed[ChangedIndex] == 0 || Changed[ChangedIndex] > LastChangedArrayHandle);
			}
			CmdIndex = Cmd.EndCmd - 1;	// Jump past children of this array (the -1 because of the ++ in the for loop)
			continue;
		}

		if (Handle == Changed[ChangedIndex])
		{
			const int32 LastChangedArrayHandle = Changed[ChangedIndex];
			ChangedIndex++;
			check(Changed[ChangedIndex] == 0 || Changed[ChangedIndex] > LastChangedArrayHandle);
		}
	}

	return Handle;
}

void FRepLayout::SanityCheckChangeList(const FConstRepObjectDataBuffer Data, TArray<uint16> & Changed) const
{
	int32 ChangedIndex = 0;
	SanityCheckChangeList_r(0, Cmds.Num() - 1, Data, Changed, ChangedIndex, 0);
	check(Changed[ChangedIndex] == 0);
}

struct FDiffPropertiesSharedParams
{
	const ERepParentFlags PropertyFlags;
	const EDiffPropertiesFlags DiffFlags;
	TArray<FProperty*>* RepNotifies;
	const TArray<FRepParentCmd>& Parents;
	const TArray<FRepLayoutCmd>& Cmds;
	const TMap<FRepLayoutCmd*, TArray<FRepLayoutCmd>>& NetSerializeLayouts;
};

template<ERepDataBufferType DestinationType, ERepDataBufferType SourceType>
struct TDiffPropertiesStackParams
{
	TRepDataBuffer<DestinationType> Destination;
	TConstRepDataBuffer<SourceType> Source;
	const uint16 StartCmd;
	const uint16 EndCmd;
};

template<ERepDataBufferType DestinationType, ERepDataBufferType SourceType>
static bool DiffProperties_r(FDiffPropertiesSharedParams& Params, TDiffPropertiesStackParams<DestinationType, SourceType>& StackParams)
{
	// Note, it's never possible for the Source or Destination data to be null.
	// At the top level, both will always be valid (pointing to the memory of each element in each respective buffer).
	// As we recurse, if we detect the size of the Arrays is different we'll either:
	//	1. Bail out if we're not syncing properties.
	//	2. Resize the Destination array to match the Source Array, guaranteeing both are allocated.

	check(StackParams.Source);
	check(StackParams.Destination);

	const bool bSyncProperties = EnumHasAnyFlags(Params.DiffFlags, EDiffPropertiesFlags::Sync);
	bool bDifferent = false;

	for (uint16 CmdIndex = StackParams.StartCmd; CmdIndex < StackParams.EndCmd; ++CmdIndex)
	{
		const FRepLayoutCmd& Cmd = Params.Cmds[CmdIndex];
		const FRepParentCmd& Parent = Params.Parents[Cmd.ParentIndex];

		if (EnumHasAnyFlags(Parent.Flags, ERepParentFlags::IsCustomDelta))
		{
			CmdIndex = Parent.CmdEnd - 1;
			continue;
		}

		check(ERepLayoutCmdType::Return != Cmd.Type);

		if (ERepLayoutCmdType::DynamicArray == Cmd.Type)
		{
			// This will ensure that we've skipped passed the array's properties.
			const uint16 ArrayStartCmd = CmdIndex + 1;
			const uint16 ArrayEndCmd = Cmd.EndCmd - 1;
			CmdIndex = ArrayEndCmd;

			FScriptArray* SourceArray = (FScriptArray*)(StackParams.Source + Cmd).Data;
			FScriptArray* DestinationArray = (FScriptArray*)(StackParams.Destination + Cmd).Data;

			if (SourceArray->Num() != DestinationArray->Num())
			{
				bDifferent = true;
				if (!bSyncProperties)
				{
					UE_LOGF(LogRep, Warning, "FDiffPropertiesImpl: Array sizes different: %ls %i / %i", *Cmd.Property->GetFullName(), SourceArray->Num(), DestinationArray->Num());
					continue;
				}
				else if (!EnumHasAnyFlags(Parent.Flags, Params.PropertyFlags))
				{
					continue;
				}

				// Make the destination state match the source state
				FScriptArrayHelper DestinationArrayHelper((FArrayProperty *)Cmd.Property, DestinationArray);
				DestinationArrayHelper.Resize(SourceArray->Num());
			}

			decltype(StackParams.Destination) ArrayDestinationData(DestinationArray->GetData());
			decltype(StackParams.Source) ArraySourceData(SourceArray->GetData());

			typename TDecay<decltype(StackParams)>::Type ArrayStackParams{
				ArrayDestinationData,
				ArraySourceData,
				ArrayStartCmd,
				ArrayEndCmd
			};

			for (int32 i = 0; i < SourceArray->Num(); ++i)
			{
				const int32 ElementOffset = i * Cmd.ElementSize;

				ArrayStackParams.Source = ArraySourceData + ElementOffset;
				ArrayStackParams.Destination = ArrayDestinationData + ElementOffset;
				bDifferent |= DiffProperties_r(Params, ArrayStackParams);
			}
		}
		else
		{
			// Make the shadow state match the actual state at the time of send
			const bool bPropertyHasRepNotifies = Params.RepNotifies && INDEX_NONE != Parent.RepNotifyNumParams;
			if ((bPropertyHasRepNotifies && Parent.RepNotifyCondition == REPNOTIFY_Always) || !PropertiesAreIdentical(Cmd, StackParams.Source + Cmd, StackParams.Destination + Cmd, Params.NetSerializeLayouts))
			{
				bDifferent = true;
				if (!bSyncProperties)
				{
					UE_LOGF(LogRep, Warning, "DiffProperties_r: Property different: %ls", *Cmd.Property->GetFullName());
					continue;
				}
				else if (!EnumHasAnyFlags(Parent.Flags, Params.PropertyFlags))
				{
					continue;
				}

				StoreProperty(Cmd, StackParams.Destination + Cmd, StackParams.Source + Cmd);

				if (bPropertyHasRepNotifies)
				{
					Params.RepNotifies->AddUnique(Parent.Property);
				}
			}
			else
			{
				UE_CLOGF(LogSkippedRepNotifies > 0, LogRep, Display, "FDiffPropertiesImpl: Skipping RepNotify because values are the same: %ls", *Cmd.Property->GetFullName());
			}
		}
	}

	return bDifferent;
}

template<ERepDataBufferType DestinationType, ERepDataBufferType SourceType>
bool FRepLayout::DiffProperties(
	TArray<FProperty*>* RepNotifies,
	TRepDataBuffer<DestinationType> Destination,
	TConstRepDataBuffer<SourceType> Source,
	const EDiffPropertiesFlags DiffFlags) const
{
	if (IsEmpty())
	{
		return false;
	}

	// Currently, only lifetime properties init from their defaults, so default to that,
	// but also diff conditional properties if requested.
	ERepParentFlags ParentPropertyFlags = ERepParentFlags::IsLifetime;
	if (EnumHasAnyFlags(DiffFlags, EDiffPropertiesFlags::IncludeConditionalProperties))
	{
		ParentPropertyFlags |= ERepParentFlags::IsConditional;
	}

	FDiffPropertiesSharedParams Params{
		ParentPropertyFlags,
		DiffFlags,
		RepNotifies,
		Parents,
		Cmds,
		NetSerializeLayouts
	};

	TDiffPropertiesStackParams<DestinationType, SourceType> StackParams{
		Destination,
		Source,
		0u,
		static_cast<uint16>(Cmds.Num() - 1)
	};

	return DiffProperties_r(Params, StackParams);
}

struct FDiffStablePropertiesSharedParams
{
	TArray<FProperty*>* RepNotifies;
	TArray<UObject*>* ObjReferences;
	const TArray<FRepParentCmd>& Parents;
	const TArray<FRepLayoutCmd>& Cmds;
	const TMap<FRepLayoutCmd*, TArray<FRepLayoutCmd>>& NetSerializeLayouts;
};

template<ERepDataBufferType DestinationType, ERepDataBufferType SourceType>
struct TDiffStablePropertiesStackParams
{
	TRepDataBuffer<DestinationType> Destination;
	TConstRepDataBuffer<SourceType> Source;
	const int32 StartCmd;
	const int32 EndCmd;
};

template<ERepDataBufferType DestinationType, ERepDataBufferType SourceType>
static bool DiffStableProperties_r(FDiffStablePropertiesSharedParams& Params, TDiffStablePropertiesStackParams<DestinationType, SourceType>& StackParams)
{
	// Note, it's never possible for the Source or Destination data to be null.
	// At the top level, both will always be valid (pointing to the memory of each element in each respective buffer).
	// As we recurse, if we detect the size of the Arrays is different we'll either:
	//	1. Bail out if we're not syncing properties.
	//	2. Resize the Destination array to match the Source Array, guaranteeing both are allocated.

	bool bDifferent = false;

	for (uint16 CmdIndex = StackParams.StartCmd; CmdIndex < StackParams.EndCmd; ++CmdIndex)
	{
		const FRepLayoutCmd& Cmd = Params.Cmds[CmdIndex];
		const FRepParentCmd& Parent = Params.Parents[Cmd.ParentIndex];

		if (EnumHasAnyFlags(Parent.Flags, ERepParentFlags::IsCustomDelta))
		{
			CmdIndex = Parent.CmdEnd - 1;
			continue;
		}

		check(ERepLayoutCmdType::Return != Cmd.Type);

		if (ERepLayoutCmdType::DynamicArray == Cmd.Type)
		{
			// This will ensure that we've skipped passed the array's properties.
			const uint16 ArrayStartCmd = CmdIndex + 1;
			const uint16 ArrayEndCmd = Cmd.EndCmd - 1;
			CmdIndex = ArrayEndCmd;

			FScriptArray* SourceArray = (FScriptArray*)(StackParams.Source + Cmd).Data;
			FScriptArray* DestinationArray = (FScriptArray*)(StackParams.Destination + Cmd).Data;

			if (SourceArray->Num() != DestinationArray->Num())
			{
				bDifferent = true;

				if (!EnumHasAnyFlags(Parent.Flags, ERepParentFlags::IsLifetime))
				{
					// Currently, only lifetime properties init from their defaults
					continue;
				}

				// Do not adjust source data, only the destination
				FScriptArrayHelper DestinationArrayHelper((FArrayProperty *)Cmd.Property, DestinationArray);
				DestinationArrayHelper.Resize(SourceArray->Num());
			}

			decltype(StackParams.Destination) ArrayDestinationData(DestinationArray->GetData());
			decltype(StackParams.Source) ArraySourceData(SourceArray->GetData());

			typename TDecay<decltype(StackParams)>::Type ArrayStackParams{
				ArrayDestinationData,
				ArraySourceData,
				ArrayStartCmd,
				ArrayEndCmd
			};

			for (int32 i = 0; i < SourceArray->Num(); ++i)
			{
				const int32 ElementOffset = i * Cmd.ElementSize;

				ArrayStackParams.Source = ArraySourceData + ElementOffset;
				ArrayStackParams.Destination = ArrayDestinationData + ElementOffset;
				bDifferent |= DiffStableProperties_r(Params, ArrayStackParams);
			}
		}
		else
		{
			if (!PropertiesAreIdentical(Cmd, StackParams.Destination + Cmd, StackParams.Source + Cmd, Params.NetSerializeLayouts))
			{
				bDifferent = true;

				if (!EnumHasAnyFlags(Parent.Flags, ERepParentFlags::IsLifetime))
				{
					// Currently, only lifetime properties init from their defaults
					continue;
				}
				else if (Cmd.Property->HasAnyPropertyFlags(CPF_Transient))
				{
					// skip transient properties
					continue;
				}

				if (Cmd.Type == ERepLayoutCmdType::PropertyObject ||
					Cmd.Type == ERepLayoutCmdType::PropertyWeakObject ||
					Cmd.Type == ERepLayoutCmdType::PropertySoftObject)
				{
					if (FObjectPropertyBase* ObjProperty = CastFieldChecked<FObjectPropertyBase>(Cmd.Property))
					{
						UObject* ObjValue = ObjProperty->GetObjectPropertyValue(StackParams.Source + Cmd);

						const bool bIsActor = ObjProperty->PropertyClass && ObjProperty->PropertyClass->IsChildOf(AActor::StaticClass());
						const bool bIsActorComponent = ObjProperty->PropertyClass && ObjProperty->PropertyClass->IsChildOf(UActorComponent::StaticClass());

						const bool bNetStartupActor = bIsActor && ObjValue && Cast<AActor>(ObjValue)->IsNetStartupActor();
						const bool bIsReplicatedActor = bIsActor && ObjValue && Cast<AActor>(ObjValue)->GetIsReplicated();

						// This is similar to AActor::IsNameStableForNetworking but we want to ignore forced net addressable objects for now. Explicitly call the UObject implementation.
						const bool bStableForNetworking = bNetStartupActor || (ObjValue && (ObjValue->UObject::IsNameStableForNetworking()));

						if (bIsReplicatedActor || bIsActorComponent)
						{
							// skip replicated actor and component references
							continue;
						}

						if (ObjValue)
						{
							if (!bStableForNetworking)
							{
								// skip object references without a stable name
								continue;
							}

							if (Params.ObjReferences)
							{
								Params.ObjReferences->AddUnique(ObjValue);
							}
						}
					}
				}
				else if (Cmd.Type == ERepLayoutCmdType::PropertyInterface)
				{
					if (FInterfaceProperty* InterfaceProperty = CastFieldChecked<FInterfaceProperty>(Cmd.Property))
					{
						if (UObject* InterfaceObjValue = InterfaceProperty->GetPropertyValue(StackParams.Source + Cmd).GetObject())
						{
							if (InterfaceObjValue->GetClass() && (InterfaceObjValue->GetClass()->IsChildOf(AActor::StaticClass()) || InterfaceObjValue->GetClass()->IsChildOf(UActorComponent::StaticClass())))
							{
								// skip actor and component references
								continue;
							}

							// Explicitly call the UObject implementation to mirror object property behavior above.
							if (!InterfaceObjValue->UObject::IsNameStableForNetworking())
							{
								// skip object references without a stable name
								continue;
							}

							if (Params.ObjReferences)
							{
								Params.ObjReferences->AddUnique(InterfaceObjValue);
							}
						}
					}
				}

				StoreProperty(Cmd, StackParams.Destination + Cmd, StackParams.Source + Cmd);

				if (Params.RepNotifies && INDEX_NONE != Parent.RepNotifyNumParams)
				{
					Params.RepNotifies->AddUnique(Parent.Property);
				}
			}
		}
	}

	return bDifferent;
}

template<ERepDataBufferType DestinationType, ERepDataBufferType SourceType>
bool FRepLayout::DiffStableProperties(
	TArray<FProperty*>* RepNotifies,
	TArray<UObject*>* ObjReferences,
	TRepDataBuffer<DestinationType> Destination,
	TConstRepDataBuffer<SourceType> Source) const
{
	FDiffStablePropertiesSharedParams Params{
		RepNotifies,
		ObjReferences,
		Parents,
		Cmds,
		NetSerializeLayouts
	};

	TDiffStablePropertiesStackParams<DestinationType, SourceType> StackParams{
		Destination,
		Source,
		0,
		Cmds.Num() - 1
	};

	return DiffStableProperties_r(Params, StackParams);
}

static FName NAME_Vector_NetQuantize100("Vector_NetQuantize100");
static FName NAME_Vector_NetQuantize10("Vector_NetQuantize10");
static FName NAME_Vector_NetQuantizeNormal("Vector_NetQuantizeNormal");
static FName NAME_Vector_NetQuantize("Vector_NetQuantize");
static FName NAME_UniqueNetIdRepl("UniqueNetIdRepl");
static FName NAME_RepMovement("RepMovement");

struct FInitFromPropertySharedParams
{
	TArray<FRepLayoutCmd>& Cmds;
	const UNetConnection* ServerConnection;
	const int32 ParentIndex;
	FRepParentCmd& Parent;
	bool bHasObjectProperties = false;
	bool bHasNetSerializeProperties = false;
	TMap<int32, TArray<FRepLayoutCmd>>* NetSerializeLayouts = nullptr;
};

struct FInitFromPropertyStackParams
{
	FProperty* Property;
	int32 Offset;
	int32 RelativeHandle;
	uint32 ParentChecksum;
	int32 StaticArrayIndex;
	FName RecursingNetSerializeStruct = NAME_None;
	bool bNetSerializeStructWithObjects = false;
};

static uint32 GetRepLayoutCmdCompatibleChecksum(
	FInitFromPropertySharedParams& SharedParams,
	const FInitFromPropertyStackParams& StackParams)
{
	return GetRepLayoutCmdCompatibleChecksum(StackParams.Property, SharedParams.ServerConnection, StackParams.StaticArrayIndex, StackParams.ParentChecksum);
}

static uint32 AddPropertyCmd(
	FInitFromPropertySharedParams& SharedParams,
	const FInitFromPropertyStackParams& StackParams)
{
	FRepLayoutCmd & Cmd = SharedParams.Cmds.AddZeroed_GetRef();

	Cmd.Property = StackParams.Property;
	Cmd.Type = ERepLayoutCmdType::Property;		// Initially set to generic type
	Cmd.Offset = StackParams.Offset;
	Cmd.ElementSize = Cmd.Property->GetElementSize();
	Cmd.RelativeHandle = StackParams.RelativeHandle;
	Cmd.ParentIndex = SharedParams.ParentIndex;
	Cmd.CompatibleChecksum = GetRepLayoutCmdCompatibleChecksum(SharedParams, StackParams);

	FProperty* UnderlyingProperty = Cmd.Property;
	if (FEnumProperty* EnumProperty = CastField<FEnumProperty>(UnderlyingProperty))
	{
		UnderlyingProperty = EnumProperty->GetUnderlyingProperty();
	}

	// Try to special case to custom types we know about
	if (UnderlyingProperty->IsA(FStructProperty::StaticClass()))
	{
		FStructProperty* StructProp = CastField<FStructProperty>(UnderlyingProperty);
		UScriptStruct* Struct = StructProp->Struct;
		Cmd.Flags |= ERepLayoutCmdFlags::IsStruct;

		if (Struct->GetFName() == NAME_Vector)
		{
			Cmd.Type = ERepLayoutCmdType::PropertyVector;
		}
		else if (Struct->GetFName() == NAME_Rotator)
		{
			Cmd.Type = ERepLayoutCmdType::PropertyRotator;
		}
		else if (Struct->GetFName() == NAME_Plane)
		{
			Cmd.Type = ERepLayoutCmdType::PropertyPlane;
		}
		else if (Struct->GetFName() == NAME_Vector_NetQuantize100)
		{
			Cmd.Type = ERepLayoutCmdType::PropertyVector100;
		}
		else if (Struct->GetFName() == NAME_Vector_NetQuantize10)
		{
			Cmd.Type = ERepLayoutCmdType::PropertyVector10;
		}
		else if (Struct->GetFName() == NAME_Vector_NetQuantizeNormal)
		{
			Cmd.Type = ERepLayoutCmdType::PropertyVectorNormal;
		}
		else if (Struct->GetFName() == NAME_Vector_NetQuantize)
		{
			Cmd.Type = ERepLayoutCmdType::PropertyVectorQ;
		}
		else if (Struct->GetFName() == NAME_UniqueNetIdRepl)
		{
			Cmd.Type = ERepLayoutCmdType::PropertyNetId;
		}
		else if (Struct->GetFName() == NAME_RepMovement)
		{
			Cmd.Type = ERepLayoutCmdType::RepMovement;
		}
		else if (StackParams.bNetSerializeStructWithObjects)
		{
			Cmd.Type = ERepLayoutCmdType::NetSerializeStructWithObjectReferences;
		}
		else
		{
			UE_LOGF(LogRep, VeryVerbose, "AddPropertyCmd: Falling back to default type for property [%ls]", *Cmd.Property->GetFullName());
		}
	}
	else if (UnderlyingProperty->IsA(FBoolProperty::StaticClass()))
	{
		const FBoolProperty* BoolProperty = static_cast<FBoolProperty*>(UnderlyingProperty);
		Cmd.Type = BoolProperty->IsNativeBool() ? ERepLayoutCmdType::PropertyNativeBool : ERepLayoutCmdType::PropertyBool;
	}
	else if (UnderlyingProperty->IsA(FFloatProperty::StaticClass()))
	{
		Cmd.Type = ERepLayoutCmdType::PropertyFloat;
	}
	else if (UnderlyingProperty->IsA(FIntProperty::StaticClass()))
	{
		Cmd.Type = ERepLayoutCmdType::PropertyInt;
	}
	else if (UnderlyingProperty->IsA(FByteProperty::StaticClass()))
	{
		Cmd.Type = ERepLayoutCmdType::PropertyByte;
	}
	else if (UnderlyingProperty->IsA(FObjectPropertyBase::StaticClass()))
	{
		SharedParams.bHasObjectProperties = true;
		if (UnderlyingProperty->IsA(FSoftObjectProperty::StaticClass()))
		{
			Cmd.Type = ERepLayoutCmdType::PropertySoftObject;
		}
		else if (UnderlyingProperty->IsA(FWeakObjectProperty::StaticClass()))
		{
			Cmd.Type = ERepLayoutCmdType::PropertyWeakObject;
		}
		else
		{
			Cmd.Type = ERepLayoutCmdType::PropertyObject;
		}
	}
	else if (UnderlyingProperty->IsA(FInterfaceProperty::StaticClass()))
	{
		Cmd.Type = ERepLayoutCmdType::PropertyInterface;
	}
	else if (UnderlyingProperty->IsA(FNameProperty::StaticClass()))
	{
		Cmd.Type = ERepLayoutCmdType::PropertyName;
	}
	else if (UnderlyingProperty->IsA(FUInt32Property::StaticClass()))
	{
		Cmd.Type = ERepLayoutCmdType::PropertyUInt32;
	}
	else if (UnderlyingProperty->IsA(FUInt64Property::StaticClass()))
	{
		Cmd.Type = ERepLayoutCmdType::PropertyUInt64;
	}
	else if (UnderlyingProperty->IsA(FStrProperty::StaticClass()))
	{
		Cmd.Type = ERepLayoutCmdType::PropertyString;
	}
	else
	{
		UE_LOGF(LogRep, VeryVerbose, "AddPropertyCmd: Falling back to default type for property [%ls]", *Cmd.Property->GetFullName());
	}

	// Cannot write a shared version of a property that depends on per-connection data (the PackageMap).
	// Includes object pointers and structs with custom NetSerialize functions (unless they opt in)
	// Also skip writing the RemoteRole since it can be modified per connection in FObjectReplicator
	if (Cmd.Property->SupportsNetSharedSerialization() && (Cmd.Property->GetFName() != NAME_RemoteRole))
	{
		Cmd.Flags |= ERepLayoutCmdFlags::IsSharedSerialization;
	}

	return Cmd.CompatibleChecksum;
}

static FORCEINLINE uint32 AddArrayCmd(
	FInitFromPropertySharedParams& SharedParams,
	const FInitFromPropertyStackParams StackParams)
{
	FRepLayoutCmd& Cmd = SharedParams.Cmds.AddZeroed_GetRef();

	Cmd.Type = ERepLayoutCmdType::DynamicArray;
	Cmd.Property = StackParams.Property;
	Cmd.Offset = StackParams.Offset;
	Cmd.ElementSize = static_cast<FArrayProperty*>(StackParams.Property)->Inner->GetElementSize();
	Cmd.RelativeHandle = StackParams.RelativeHandle;
	Cmd.ParentIndex = SharedParams.ParentIndex;
	Cmd.CompatibleChecksum = GetRepLayoutCmdCompatibleChecksum(SharedParams, StackParams);

	return Cmd.CompatibleChecksum;
}

static FORCEINLINE void AddReturnCmd(TArray<FRepLayoutCmd>& Cmds)
{
	Cmds.AddZeroed_GetRef().Type = ERepLayoutCmdType::Return;
}

enum class ERepBuildType
{
	Class,
	Function,
	Struct
};

template<ERepBuildType BuildType>
static FORCEINLINE const int32 GetOffsetForProperty(const FProperty& Property)
{
	return Property.GetOffset_ForGC();
}

template<>
const FORCEINLINE int32 GetOffsetForProperty<ERepBuildType::Function>(const FProperty& Property)
{
	return Property.GetOffset_ForUFunction();
}

template<ERepBuildType BuildType>
static int32 InitFromProperty_r(
	FInitFromPropertySharedParams& SharedParams,
	FInitFromPropertyStackParams StackParams);

template<ERepBuildType BuildType>
static int32 InitFromStructProperty(
	FInitFromPropertySharedParams& SharedParams,
	FInitFromPropertyStackParams StackParams,
	const FStructProperty* const StructProp,
	const UScriptStruct* const Struct)
{
	// Track properties so we can ensure they are sorted by offsets at the end
	// TODO: Do these actually need to be sorted?
	TArray<FProperty*> NetProperties;

	for (TFieldIterator<FProperty> It(Struct); It; ++It)
	{
		if ((It->PropertyFlags & CPF_RepSkip))
		{
			continue;
		}

		NetProperties.Add(*It);
	}

	// Sort NetProperties by memory offset
	struct FCompareUFieldOffsets
	{
		FORCEINLINE bool operator()(FProperty* A, FProperty* B) const
		{
			const int32 AOffset = GetOffsetForProperty<BuildType>(*A);
			const int32 BOffset = GetOffsetForProperty<BuildType>(*B);

			// Ensure stable sort
			if (AOffset == BOffset)
			{
				return A->GetName() < B->GetName();
			}

			return AOffset < BOffset;
		}
	};

	Algo::Sort(NetProperties, FCompareUFieldOffsets());

	const uint32 StructChecksum = GetRepLayoutCmdCompatibleChecksum(SharedParams, StackParams);

	for (int32 i = 0; i < NetProperties.Num(); i++)
	{
		for (int32 j = 0; j < NetProperties[i]->ArrayDim; j++)
		{
			const int32 ArrayElementOffset = j * NetProperties[i]->GetElementSize();

			FInitFromPropertyStackParams NewStackParams{
				/*Property=*/NetProperties[i],
				/*Offset=*/StackParams.Offset + ArrayElementOffset,
				/*RelativeHandle=*/StackParams.RelativeHandle,
				/*ParentChecksum=*/StructChecksum,
				/*StaticArrayIndex=*/j,
				/*RecursingNetSerializeStruct=*/StackParams.RecursingNetSerializeStruct
			};

			StackParams.RelativeHandle = InitFromProperty_r<BuildType>(SharedParams, NewStackParams);
		}
	}

	return StackParams.RelativeHandle;
}

template<ERepBuildType BuildType>
static int32 InitFromProperty_r(
	FInitFromPropertySharedParams& SharedParams,
	FInitFromPropertyStackParams StackParams)
{
	if (FArrayProperty* ArrayProp = CastField<FArrayProperty>(StackParams.Property))
	{
		const int32 CmdStart = SharedParams.Cmds.Num();
		SharedParams.Parent.Flags |= ERepParentFlags::HasDynamicArrayProperties;

		++StackParams.RelativeHandle;
		StackParams.Offset += GetOffsetForProperty<BuildType>(*ArrayProp);

		const uint32 ArrayChecksum = AddArrayCmd(SharedParams, StackParams);

		FInitFromPropertyStackParams NewStackParams{
			/*Property=*/ArrayProp->Inner,
			/*Offset=*/0,
			/*RelativeHandle=*/0,
			/*ParentChecksum=*/ArrayChecksum,
			/*StaticArrayIndex=*/0,
			/*RecursingNetSerializeStruct=*/StackParams.RecursingNetSerializeStruct
		};

		InitFromProperty_r<BuildType>(SharedParams, NewStackParams);

		AddReturnCmd(SharedParams.Cmds);

		const int32 CmdEnd = SharedParams.Cmds.Num();
		SharedParams.Cmds[CmdStart].EndCmd = CmdEnd;		// Patch in the offset to jump over our array inner elements

		// Array commands will have their array property, the layout of the inner property, and a terminator.
		// That means if we only have 2 commands, the array's inner propertry had no replicated properties of
		// its own.
		if (CmdEnd - CmdStart <= 2)
		{
			SharedParams.Cmds[CmdStart].Flags |= ERepLayoutCmdFlags::IsEmptyArrayStruct;
			UE_LOGF(LogRep, Warning, "InitFromProperty_r: Array property has empty inner struct: Outer=%ls, Array=%ls, Inner=%ls",
				*ArrayProp->Owner.GetName(), *ArrayProp->GetName(), *ArrayProp->Inner->GetName());
		}
	}
	else if (FStructProperty* StructProp = CastField<FStructProperty>(StackParams.Property))
	{
		UScriptStruct* Struct = StructProp->Struct;

		StackParams.Offset += GetOffsetForProperty<BuildType>(*StructProp);
		if (EnumHasAnyFlags(Struct->StructFlags, STRUCT_NetSerializeNative))
		{
			UE_CLOGF(EnumHasAnyFlags(Struct->StructFlags, STRUCT_NetDeltaSerializeNative), LogRep, Warning, "RepLayout InitFromProperty_r: Struct marked both NetSerialize and NetDeltaSerialize: %ls", *StructProp->GetName());

			SharedParams.bHasNetSerializeProperties = true;
			if (ERepBuildType::Class == BuildType && GbTrackNetSerializeObjectReferences && nullptr != SharedParams.NetSerializeLayouts && !EnumHasAnyFlags(Struct->StructFlags, STRUCT_IdenticalNative))
			{
				// We can't directly rely on FProperty::Identical because it's not safe for GC'd objects.
				// So, we'll recursively build up set of layout commands for this struct, and if any
				// are Objects, we'll use that for storing items in Shadow State and comparison.
				// Otherwise, we'll fall back to the old behavior.
				const int32 PrevCmdNum = SharedParams.Cmds.Num();

				TArray<FRepLayoutCmd> TempCmds;
				TArray<FRepLayoutCmd>* NewCmds = &TempCmds;
				
				FInitFromPropertyStackParams NewStackParams{
					/*Property=*/StackParams.Property,
					/*Offset=*/0,
					/*RelativeHandle=*/StackParams.RelativeHandle,
					/*ParentChecksum=*/StackParams.ParentChecksum,
					/*StaticArrayIndex=*/StackParams.StaticArrayIndex,
					/*RecursingNetSerialize=*/StructProp->GetFName()
					
				};

				if (StackParams.RecursingNetSerializeStruct != NAME_None)
				{
					NewCmds = &SharedParams.Cmds;
					NewStackParams.RelativeHandle = 0;
				}

				FInitFromPropertySharedParams NewSharedParams{
					/*Cmds=*/*NewCmds,
					/*ServerConnection=*/SharedParams.ServerConnection,
					/*ParentIndex=*/SharedParams.ParentIndex,
					/*Parent=*/SharedParams.Parent,
					/*bHasObjectProperties=*/false,
					/*bHasNetSerializeProperties=*/false,
					/*NetSerializeLayouts=*/SharedParams.NetSerializeLayouts
				};

				const int32 NetSerializeStructOffset = InitFromStructProperty<BuildType>(NewSharedParams, NewStackParams, StructProp, Struct);

				if (StackParams.RecursingNetSerializeStruct == NAME_None)
				{
					if (NewSharedParams.bHasObjectProperties)
					{
						// If this is a top level Net Serialize Struct, and we found any any objects,
						// then we need to make sure this is tracked in our map.
						SharedParams.NetSerializeLayouts->Add(SharedParams.Cmds.Num(), MoveTemp(TempCmds));
						StackParams.bNetSerializeStructWithObjects = true;
					}
				}
				else if (!NewSharedParams.bHasObjectProperties)
				{
					// If this wasn't a top level Net Serialize Struct, and we didn't find any objects,
					// we need to remove any nested entries we added to the Net Serialize Struct's layout.
					// Instead, we'll assume this layout is FProperty safe, and add it as single command (below).
					SharedParams.Cmds.SetNum(PrevCmdNum);
				}
				else
				{
					// This wasn't a top level Net Serialize Struct, but we did find some objects.
					// We want to keep the layout we generated, so keep that layout
					return NetSerializeStructOffset;
				}
			}

			++StackParams.RelativeHandle;
			AddPropertyCmd(SharedParams, StackParams);

			return StackParams.RelativeHandle;
		}

		
		return InitFromStructProperty<BuildType>(SharedParams, StackParams, StructProp, Struct);
	}
	else
	{
		// Add actual property
		++StackParams.RelativeHandle;
		StackParams.Offset += GetOffsetForProperty<BuildType>(*StackParams.Property);

		AddPropertyCmd(SharedParams, StackParams);

		if (StackParams.RecursingNetSerializeStruct != NAME_None &&
			ERepLayoutCmdType::Property == SharedParams.Cmds.Last().Type)
		{
			TArray<const FStructProperty*> SubProperties;
			if (StackParams.Property->ContainsObjectReference(SubProperties))
			{
				// This error indicates that we're seeing a property within some NetSerialize struct
				// that references a UObject, but isn't handle by *normal* replication means
				// (e.g., this could be a map or a set that we just need to compare or store, but not serialize).
				// That's dangerous, because we will end up storing the Object Reference in the Shadow State,
				// and it could be garbage the next time Property->Identical is called, leading to undefined behavior.
				//
				// The easiest fix is to convert the StructProperty's Struct Type to using either a native identity check
				// or a native equality operator, and manually comparing just the pointer values for the object.

				UE_LOGF(LogRep, Warning,
					"InitFromProperty_r: Found NetSerialize Struct Property that contains a nested UObject reference that is not tracked for replication. StructProperty=%ls, NestedProperty=%ls",
					*StackParams.RecursingNetSerializeStruct.ToString(), *StackParams.Property->GetPathName());
			}
		}
	}

	return StackParams.RelativeHandle;
}

static FORCEINLINE uint16 AddParentProperty(
	TArray<FRepParentCmd>& Parents,
	FProperty* Property,
	int32 ArrayIndex)
{
	return Parents.Emplace(Property, ArrayIndex);
}

/** Setup some flags on our parent properties, so we can handle them properly later.*/
static FORCEINLINE void SetupRepStructFlags(FRepParentCmd& Parent, const bool bSkipCustomDeltaCheck)
{
	if (FStructProperty* StructProperty = CastField<FStructProperty>(Parent.Property))
	{
		UScriptStruct* Struct = StructProperty->Struct;

		Parent.Flags |= ERepParentFlags::IsStructProperty;

		if (!bSkipCustomDeltaCheck && EnumHasAnyFlags(Struct->StructFlags, STRUCT_NetDeltaSerializeNative))
		{
			Parent.Flags |= ERepParentFlags::IsCustomDelta;

			if (Struct->IsChildOf(FFastArraySerializer::StaticStruct()))
			{
				Parent.Flags |= ERepParentFlags::IsFastArray;
			}
		}

		if (EnumHasAnyFlags(Struct->StructFlags, STRUCT_NetSerializeNative))
		{
			Parent.Flags |= ERepParentFlags::IsNetSerialize;
		}
	}

	if (EnumHasAnyFlags(Parent.Property->PropertyFlags, CPF_ZeroConstructor))
	{
		Parent.Flags |= ERepParentFlags::IsZeroConstructible;
	}
}

/**
* Dynamic Array Properties:
*		These will have their memory allocated separate from the actual Shadow Buffer.
*		Conceptually, their layout in the Shadow Buffer is a separate sub-RepLayout with only one Parent Property
*		and potentially multiple children.
*
* Static Array Properties:
*		These will have their memory allocated inline in the shadow buffer.
*		Due to the way we currently initialize, construct, and destruct elements, we need
*		to allocate the entire size of the elements in these arrays.
*		@see InitProperties, ConstructProperties, DestructProperties.
*
* Struct Properties are broken into 3 main cases:
*
*		NetDeltaSerialize:
*			These structs will not have Child Rep Commands, but they will still have Parent Commands.
*			This is because we generally don't care about their Memory Layout, but we need to
*			be able to initialize them properly.
*
*		NetSerialize:
*			These structs will have a single Child Rep Command for the FStructProperty.
*			Similar to NetDeltaSerialize, we don't really care about the memory layout of NetSerialize
*			structs, but we still need to know where they live so we can diff them, etc.
*
*		Everything Else:
*			These structs will have potentially many Child Rep Commands, as we flatten their structure.
*			Note, there **will not** be a Child Rep Command for the actual owning property.
*			We do care about the memory layout in this case, because the RepLayout will be
*			completely in charge of serialization, comparisons, etc.
*
*		For every case, we will still end up allocating the complete struct into the shadow state.
*/
template<bool bAlreadyAligned>
static void BuildShadowOffsets_r(TArray<FRepLayoutCmd>::TIterator& CmdIt, int32& ShadowOffset)
{
	check(CmdIt);
	check(ERepLayoutCmdType::Return != CmdIt->Type);

	// Note, the only time we should see a StructProperty is if we have a NetSerialize struct.
	// Custom Delta Serialize structs won't have an associated RepLayout command,
	// and normal structs will flatten their properties.
	if (CmdIt->Type == ERepLayoutCmdType::DynamicArray || EnumHasAnyFlags(CmdIt->Flags, ERepLayoutCmdFlags::IsStruct))
	{
		if (!bAlreadyAligned)
		{
			// Note, we can't use the Commands reported element size, as Array Commands
			// will have that set to their inner property size.

			ShadowOffset = Align(ShadowOffset, CmdIt->Property->GetMinAlignment());
			CmdIt->ShadowOffset = ShadowOffset;
			ShadowOffset += CmdIt->Property->GetSize();
		}

		if (CmdIt->Type == ERepLayoutCmdType::DynamicArray)
		{
			// Iterator into the array's layout.
			++CmdIt;

			for (; ERepLayoutCmdType::Return != CmdIt->Type; ++CmdIt)
			{
				CmdIt->ShadowOffset = CmdIt->Offset;
				BuildShadowOffsets_r</*bAlreadyAligned=*/true>(CmdIt, CmdIt->ShadowOffset);
			}

			check(CmdIt);
		}
	}
	else if (!bAlreadyAligned)
	{
		// This property is already aligned, and ShadowOffset should be correct and managed elsewhere.
		if (ShadowOffset > 0)
		{
			// Bools may be packed as bitfields, and if so they can be stored in the same location
			// as a previous property.
			if (ERepLayoutCmdType::PropertyBool == CmdIt->Type && CmdIt.GetIndex() > 0)
			{
				const TArray<FRepLayoutCmd>::TIterator PrevCmdIt = CmdIt - 1;
				if (ERepLayoutCmdType::PropertyBool == PrevCmdIt->Type && PrevCmdIt->Offset == CmdIt->Offset)
				{
					ShadowOffset = PrevCmdIt->ShadowOffset;
				}
			}
			else
			{
				ShadowOffset = Align(ShadowOffset, CmdIt->Property->GetMinAlignment());
			}
		}

		CmdIt->ShadowOffset = ShadowOffset;
		ShadowOffset += CmdIt->ElementSize;
	}
}

template<ERepBuildType ShadowType>
static void BuildShadowOffsets(
	UStruct* Owner,
	TArray<FRepParentCmd>& Parents,
	TArray<FRepLayoutCmd>& Cmds,
	int32& ShadowOffset)
{
	if (ShadowType == ERepBuildType::Class && !!GUsePackedShadowBuffers)
	{
		ShadowOffset = 0;

		if (0 != Parents.Num())
		{
			// Before filling out any ShadowOffset information, we'll sort the Parent Commands by alignment.
			// This has 2 main benefits:
			//	1. It will guarantee a minimal amount of wasted space when packing.
			//	2. It should generally improve cache hit rate when iterating over commands.
			//		Even though iteration of the commands won't actually be ordered anywhere else,
			//		this increases the likelihood that more shadow data fits into a single cache line.
			struct FParentCmdIndexAndAlignment
			{
				FParentCmdIndexAndAlignment(int32 ParentIndex, const FRepParentCmd& Parent):
					Index(ParentIndex),
					Alignment(Parent.Property->GetMinAlignment())
				{
				}

				const int32 Index;
				const int32 Alignment;

				// Needed for sorting.
				bool operator< (const FParentCmdIndexAndAlignment& RHS) const
				{
					return Alignment < RHS.Alignment;
				}
			};

			TArray<FParentCmdIndexAndAlignment> IndexAndAlignmentArray;
			IndexAndAlignmentArray.Reserve(Parents.Num());
			for (int32 i = 0; i < Parents.Num(); ++i)
			{
				IndexAndAlignmentArray.Emplace(i, Parents[i]);
			}

			IndexAndAlignmentArray.StableSort();

			for (int32 i = 0; i < IndexAndAlignmentArray.Num(); ++i)
			{
				const FParentCmdIndexAndAlignment& IndexAndAlignment = IndexAndAlignmentArray[i];
				FRepParentCmd& Parent = Parents[IndexAndAlignment.Index];

				if (Parent.Property->ArrayDim > 1 || EnumHasAnyFlags(Parent.Flags, ERepParentFlags::IsStructProperty))
				{
					const int32 ArrayStartParentOffset = GetOffsetForProperty<ShadowType>(*Parent.Property);

					ShadowOffset = Align(ShadowOffset, IndexAndAlignment.Alignment);

					for (int32 j = 0; j < Parent.Property->ArrayDim; ++j, ++i)
					{
						const FParentCmdIndexAndAlignment& NextIndexAndAlignment = IndexAndAlignmentArray[i];
						FRepParentCmd& NextParent = Parents[NextIndexAndAlignment.Index];

						NextParent.ShadowOffset = ShadowOffset + (GetOffsetForProperty<ShadowType>(*NextParent.Property) - ArrayStartParentOffset);

						for (auto CmdIt = Cmds.CreateIterator() + NextParent.CmdStart; CmdIt.GetIndex() < NextParent.CmdEnd; ++CmdIt)
						{
							CmdIt->ShadowOffset = ShadowOffset + (CmdIt->Offset - ArrayStartParentOffset);
							BuildShadowOffsets_r</*bAlreadyAligned*/true>(CmdIt, CmdIt->ShadowOffset);
						}
					}

					// The above loop will have advanced us one too far, so roll back.
					// This will make sure the outer loop has a chance to process the parent next time.
					--i;
					ShadowOffset += Parent.Property->GetSize();
				}
				else
				{
					check(Parent.CmdEnd > Parent.CmdStart);

					for (auto CmdIt = Cmds.CreateIterator() + Parent.CmdStart; CmdIt.GetIndex() < Parent.CmdEnd; ++CmdIt)
					{
						BuildShadowOffsets_r</*bAlreadyAligned=*/false>(CmdIt, ShadowOffset);
					}

					// We update this after we build child commands offsets, to make sure that
					// if there's any extra packing (like bitfield packing), we are aware of it.
					Parent.ShadowOffset = Cmds[Parent.CmdStart].ShadowOffset;
				}
			}
		}
	}
	else
	{
		ShadowOffset = Owner->GetPropertiesSize();

		for (auto ParentIt = Parents.CreateIterator(); ParentIt; ++ParentIt)
		{
			ParentIt->ShadowOffset = GetOffsetForProperty<ShadowType>(*ParentIt->Property);
		}

		for (auto CmdIt = Cmds.CreateIterator(); CmdIt; ++CmdIt)
		{
			CmdIt->ShadowOffset = CmdIt->Offset;
		}
	}
}

TSharedPtr<FRepLayout> FRepLayout::CreateFromClass(
	UClass* InClass,
	const UNetConnection* ServerConnection,
	const ECreateRepLayoutFlags CreateFlags)
{
	TSharedPtr<FRepLayout> RepLayout = MakeShareable<FRepLayout>(new FRepLayout());
	RepLayout->InitFromClass(InClass, ServerConnection, CreateFlags);
	return RepLayout;
}

void FRepLayout::InitFromClass(
	UClass* InObjectClass,
	const UNetConnection* ServerConnection,
	const ECreateRepLayoutFlags CreateFlags)
{
	SCOPE_CYCLE_COUNTER(STAT_RepLayout_InitFromObjectClass);

	const bool bIsPushModelEnabled = IS_PUSH_MODEL_ENABLED();
	const bool bIsObjectActor = InObjectClass->IsChildOf(AActor::StaticClass());

	if (bIsObjectActor)
	{
		Flags |= ERepLayoutFlags::IsActor;
	}

	int32 RelativeHandle = 0;
	int32 LastOffset = INDEX_NONE;
	int32 HighestCustomDeltaRepIndex = INDEX_NONE;
	TMap<int32, TArray<FRepLayoutCmd>> TempNetSerializeLayouts;

	InObjectClass->SetUpRuntimeReplicationData();
	Parents.Empty(InObjectClass->ClassReps.Num());

	for (int32 i = 0; i < InObjectClass->ClassReps.Num(); i++)
	{
		FProperty * Property = InObjectClass->ClassReps[i].Property;
		const int32 ArrayIdx = InObjectClass->ClassReps[i].Index;

#if DO_CHECK
		if (UNLIKELY(Property == nullptr))
		{
			FString Message = FString::Printf(TEXT("Class: %s | Index: %d"), *InObjectClass->GetPathName(), i);
			if (i > 0)
			{
				Message += FString::Printf(TEXT(" | PreviousClassRepProperty: %s"), *GetPathNameSafe(InObjectClass->ClassReps[i - 1].Property));
			}
			else if (i < InObjectClass->ClassReps.Num() - 1)
			{
				Message += FString::Printf(TEXT(" | NextClassRepProperty: %s"), *GetPathNameSafe(InObjectClass->ClassReps[i + 1].Property));
			}

			checkf(false, TEXT("Encountered an invalid property while creating RepLayout. This should never happen! %s"), *Message);
			return;
		}
#endif

		check(Property->PropertyFlags & CPF_Net);

		const int32 ParentHandle = AddParentProperty(Parents, Property, ArrayIdx);

		check(ParentHandle == i);
		check(Parents[i].Property->RepIndex + Parents[i].ArrayIndex == i);

		const int32 ParentOffset = Property->GetElementSize() * ArrayIdx;

		FInitFromPropertySharedParams SharedParams
		{
			/*Cmds=*/Cmds,
			/*ServerConnection=*/ServerConnection,
			/*ParentIndex=*/ParentHandle,
			/*Parent=*/Parents[ParentHandle],
			/*bHasObjectProperties=*/false,
			/*bHasNetSerializeProperties=*/false,
			/*NetSerializeLayouts=*/GbTrackNetSerializeObjectReferences ? &TempNetSerializeLayouts : nullptr,
		};

		FInitFromPropertyStackParams StackParams
		{
			/*Property=*/Property,
			/*Offset=*/ParentOffset,
			/*RelativeHandle=*/RelativeHandle,
			/*ParentChecksum=*/0,
			/*StaticArrayIndex=*/ArrayIdx
		};

		Parents[ParentHandle].CmdStart = Cmds.Num();
		RelativeHandle = InitFromProperty_r<ERepBuildType::Class>(SharedParams, StackParams);
		Parents[ParentHandle].CmdEnd = Cmds.Num();
		Parents[ParentHandle].Flags |= ERepParentFlags::IsConditional;
		Parents[ParentHandle].Offset = GetOffsetForProperty<ERepBuildType::Class>(*Property) + ParentOffset;

		if (Parents[i].CmdEnd > Parents[i].CmdStart)
		{
			check(Cmds[Parents[i].CmdStart].Offset >= LastOffset);		//>= since bool's can be combined
			LastOffset = Cmds[Parents[i].CmdStart].Offset;
		}

		// Setup flags
		SetupRepStructFlags(Parents[ParentHandle], /**bSkipCustomDeltaCheck=*/false);

		if (Property->GetPropertyFlags() & CPF_Config)
		{
			Parents[ParentHandle].Flags |= ERepParentFlags::IsConfig;
		}

		if (EnumHasAnyFlags(Parents[ParentHandle].Flags, ERepParentFlags::IsCustomDelta))
		{
			HighestCustomDeltaRepIndex = ParentHandle;
		}

		if (SharedParams.bHasNetSerializeProperties)
		{
			Parents[ParentHandle].Flags |= ERepParentFlags::HasNetSerializeProperties;
		}
		if (SharedParams.bHasObjectProperties)
		{
			Parents[ParentHandle].Flags |= ERepParentFlags::HasObjectProperties;
		}
	}

	// Make sure RemoteRole has a lower RepIndex than Role, otherwise assumptions RepLayout may break.
	static_assert((int32)AActor::ENetFields_Private::RemoteRole < (int32)AActor::ENetFields_Private::Role, "Role and RemoteRole have been rearranged in AActor. This will break assumptions in RepLayout.");

	// Make sure that our RemoteRole property actually points to RemoteRole.
	check(!bIsObjectActor || Parents[(int32)AActor::ENetFields_Private::RemoteRole].Property->GetFName() == NAME_RemoteRole);

	// Make sure that our Role property actually points to Role.
	check(!bIsObjectActor || Parents[(int32)AActor::ENetFields_Private::Role].Property->GetFName() == NAME_Role);

	AddReturnCmd(Cmds);

	if (TempNetSerializeLayouts.Num() > 0)
	{
		NetSerializeLayouts.Empty(TempNetSerializeLayouts.Num());
		for (auto It = TempNetSerializeLayouts.CreateIterator(); It; ++It)
		{
			NetSerializeLayouts.Emplace(&Cmds[It.Key()], MoveTemp(It.Value()));
		}

		NetSerializeLayouts.Shrink();
	}

	// Initialize replicated properties

	// The list of all replicated properties
	TArray<FLifetimeProperty> LifetimeProps;
	LifetimeProps.Reserve(Parents.Num());
	
	// Ask the user code for explicit settings
	UObject* Object = InObjectClass->GetDefaultObject();
	Object->GetLifetimeReplicatedProps(LifetimeProps);

	// Set default settings for properties not explicitly registered
	{
		UE::Net::FNetBitArray RegisteredProperties(InObjectClass->ClassReps.Num());
		for (const FLifetimeProperty& UserProp : LifetimeProps)
		{
			RegisteredProperties.SetBit(UserProp.RepIndex);
		}

		RegisteredProperties.ForAllUnsetBits([&LifetimeProps, InObjectClass](uint32 Index)
		{
			const uint32 PropIndex = LifetimeProps.Emplace(FLifetimeProperty(Index));

			// Check if the registration is missing if we don't automatically replicate unregistered variables
			if (!UE::Net::bAutoRegisterReplicatedProperties)
			{
				LifetimeProps[PropIndex].Condition = COND_Never;

				if (UE::Net::bEnsureForMissingProperties)
				{
					ensureMsgf(false, TEXT("Property %s::%s (SourceClass: %s) was not registered in GetLifetimeReplicatedProps. This property will not be replicated. Use DISABLE_REPLICATED_PROPERTY if not replicating was intentional."), 
							   *GetNameSafe(InObjectClass), *GetNameSafe(InObjectClass->ClassReps[Index].Property), *GetNameSafe(InObjectClass->ClassReps[Index].Property->GetOwnerClass()));
				}
			}
		});
	}

	// If there are custom delta properties we may have to change the order we traverse the replicated props.
	if (UE::Net::bReplicateCustomDeltaPropertiesInRepIndexOrder && (HighestCustomDeltaRepIndex != INDEX_NONE))
	{
		Algo::SortBy(LifetimeProps, [](const FLifetimeProperty& Element) { return Element.RepIndex; }, TLess<decltype(FLifetimeProperty::RepIndex)>());
	}

#if WITH_PUSH_MODEL
	PushModelProperties.Init(false, Parents.Num());
#endif

	// Tracks the number of (non-delta) lifetime properties so we can check that against our
	// Push Model Enabled properties.
	int32 NumberOfLifetimeProperties = 0;
	int32 NumberOfLifetimePushModelProperties = 0;
	int32 NumberOfFastArrayProperties = 0;
	int32 NumberOfFastArrayPushModelProperties = 0;

	// Use default settings when auto-registering properties
	FLifetimeProperty DefaultLifetimeProp;

	// Setup lifetime replicated properties
	for (int32 i = 0; i < LifetimeProps.Num(); i++)
	{
		const int32 ParentIndex = LifetimeProps[i].RepIndex;
		const bool bIsValidParentIndex = ensureMsgf(InObjectClass->ClassReps.IsValidIndex(ParentIndex), TEXT("LifeTimeProps[%d].RepIndex references non existing classRep %d"), i, ParentIndex);
		if (!bIsValidParentIndex)
		{
			continue;
		}

		const FRepRecord& ReplicatedProp = InObjectClass->ClassReps[ParentIndex];
	
		const bool bIsValid = ensureMsgf(Parents.IsValidIndex(ParentIndex), TEXT("ClassReps RepRecord of %s point to invalid RepIndex=%i for replicated property: %s. Parents.Num() = %d"), *GetFullNameSafe(InObjectClass), ParentIndex, *GetNameSafe(ReplicatedProp.Property), Parents.Num());
		if (!bIsValid)
		{
			continue;
		}

		FRepParentCmd& RepParentCmd = Parents[ParentIndex];

		// Find a lifetime condition defined for this replicated property
		const FLifetimeProperty* UserLifetimeProp = LifetimeProps.FindByPredicate([ParentIndex](const FLifetimeProperty& Item)
		{
			return Item.RepIndex == ParentIndex;
		});

		
		// Set the correct RepIndex just in case its referenced.
		DefaultLifetimeProp.RepIndex = ParentIndex;

		// Use Default settings if the user didn't 
		const FLifetimeProperty& LifetimeProp = UserLifetimeProp ? *UserLifetimeProp : DefaultLifetimeProp;

		// Don't bother doing any setup work for COND_Never properties.
		// These are never expected to replicate.
		if (COND_Never == LifetimeProp.Condition)
		{
			continue;
		}

		// Store the condition on the parent in case we need it
		RepParentCmd.Condition = LifetimeProp.Condition;
		RepParentCmd.RepNotifyCondition = LifetimeProp.RepNotifyCondition;

		if (UFunction* RepNotifyFunc = InObjectClass->FindFunctionByName(RepParentCmd.Property->RepNotifyFunc))
		{
			RepParentCmd.RepNotifyNumParams = RepNotifyFunc->NumParms;
		}

		if (!EnumHasAnyFlags(RepParentCmd.Flags, ERepParentFlags::IsCustomDelta))
		{
			RepParentCmd.Flags |= ERepParentFlags::IsLifetime;
			if (RepParentCmd.Condition == COND_None)
			{
				RepParentCmd.Flags &= ~ERepParentFlags::IsConditional;
			}
			else if (RepParentCmd.Condition == COND_InitialOnly)
			{
				Flags |= ERepLayoutFlags::HasInitialOnlyProperties;
			}
			else if (RepParentCmd.Condition == COND_Dynamic)
			{
				Flags |= ERepLayoutFlags::HasDynamicConditionProperties;
			}

			if (EnumHasAnyFlags(RepParentCmd.Flags, ERepParentFlags::HasNetSerializeProperties | ERepParentFlags::HasObjectProperties))
			{
				Flags |= ERepLayoutFlags::HasObjectOrNetSerializeProperties;
			}

			++NumberOfLifetimeProperties;
#if WITH_PUSH_MODEL
			if (bIsPushModelEnabled && LifetimeProp.bIsPushBased)
			{
				++NumberOfLifetimePushModelProperties;
				PushModelProperties[ParentIndex] = true;
			}
#endif
		}
		else
		{
			// We'll track all Custom Lifetime Properties here, and we'll handle Fast Array Serialization
			// specially.

			// Note, there are in engine cases where we aren't using FFastArraySerialize, but are using
			// Custom Delta (FGameplayDebuggerNetPack).

			// Also note, we still don't mark these properties as Lifetime. This should help maintain behavior
			// in Diff Properties / Compare Properties.

			if (!LifetimeCustomPropertyState)
			{
				// We can't use the number of Lifetime Properties, because that could be smaller than
				// the highest RepIndex of a Custom Delta Property, because properties may be disabled, removed,
				// or just never added.
				// For similar reasons, we don't want to use the total number of replicated properties, especially
				// if we know we'll never use anything beyond the last Custom Delta Property anyway.
				LifetimeCustomPropertyState.Reset(new FLifetimeCustomDeltaState(HighestCustomDeltaRepIndex));
			}

			// If we're a FastArraySerializer, we'll look for our replicated item type.
			// We do this by looking for an array property whose inner type is an FFastArraySerializerItem.
			// Note, this isn't perfect. With the way the interface is set up now, there's no technically
			// enforced requirements that the Array of items lives within the Fast Array Serializer, that the
			// Array of items is marked up as a FProperty, that the Array of items is not marked RepSkip,
			// or that there's not multiple arrays of FastArraySerializerItems.
			//
			// However, comments imply these, and typically they are true (certainly, any engine cases follow this).
			// Further, these layouts are only needed for the new Delta Struct Serialization feature, so this won't break backwards compat.

			bool bAddedFastArray = false;

			if (EnumHasAnyFlags(RepParentCmd.Flags, ERepParentFlags::IsFastArray))
			{
				int32 FastArrayItemArrayCmd = INDEX_NONE;

				for (int32 CmdIndex = RepParentCmd.CmdStart; CmdIndex < RepParentCmd.CmdEnd; ++CmdIndex)
				{
					const FRepLayoutCmd& Cmd = Cmds[CmdIndex];
					if (ERepLayoutCmdType::DynamicArray == Cmd.Type)
					{
						if (FStructProperty* MaybeFastArrayItemsArray = CastField<FStructProperty>(static_cast<FArrayProperty*>(Cmd.Property)->Inner))
						{
							UScriptStruct* MaybeFastArrayItem = MaybeFastArrayItemsArray->Struct;
							if (MaybeFastArrayItem->IsChildOf(FFastArraySerializerItem::StaticStruct()))
							{
								// Can't use GET_MEMBER_NAME_CHECKED because this is private.
								static const FName FastArrayDeltaFlagsName("DeltaFlags");
								static const FName FastArrayArrayReplicationKeyName(GET_MEMBER_NAME_CHECKED(FFastArraySerializer, ArrayReplicationKey));
								static const FName FastArrayItemReplicationIDName(GET_MEMBER_NAME_CHECKED(FFastArraySerializerItem, ReplicationID));

								// This better be a script struct, otherwise our flags aren't set up correctly!
								UScriptStruct* FastArray = CastChecked<UScriptStruct>(MaybeFastArrayItemsArray->GetOwnerStruct());

								LifetimeCustomPropertyState->Add(FLifetimeCustomDeltaProperty(
									/*PropertyRepIndex=*/ParentIndex,
									/*FastArrayItemsCommand=*/CmdIndex,
									/*FastArrayNumber=*/LifetimeCustomPropertyState->GetNumFastArrayProperties(),
									/*FastArrayDeltaFlagsOffset=*/FastArray->FindPropertyByName(FastArrayDeltaFlagsName)->GetOffset_ForGC(),
									/*FastArrayReplicationKeyOffset=*/FastArray->FindPropertyByName(FastArrayArrayReplicationKeyName)->GetOffset_ForGC(),
									/*FastArrayItemReplicationIdOffset=*/MaybeFastArrayItem->FindPropertyByName(FastArrayItemReplicationIDName)->GetOffset_ForGC()
								));

								++NumberOfFastArrayProperties;
#if WITH_PUSH_MODEL
								if (bIsPushModelEnabled && LifetimeProp.bIsPushBased)
								{
									++NumberOfFastArrayPushModelProperties;
									PushModelProperties[ParentIndex] = true;
								}
#endif
								bAddedFastArray = true;
								break;
							}
						}

						CmdIndex = Cmd.EndCmd - 1;
					}
				}

				if (!bAddedFastArray)
				{
					UE_LOGF(LogRep, Warning, "FRepLayout::InitFromClass: Unable to find Fast Array Item array in Fast Array Serializer: %ls", *RepParentCmd.CachedPropertyName.ToString());
				}
			}

			if (!bAddedFastArray)
			{
				LifetimeCustomPropertyState->Add(FLifetimeCustomDeltaProperty(ParentIndex));
			}
		}
	}

	if (bIsObjectActor)
	{
		// We handle remote role specially, since it can change between connections when downgraded
		// So we force it on the conditional list
		FRepParentCmd& RemoteRoleParent = Parents[(int32)AActor::ENetFields_Private::RemoteRole];
		if (RemoteRoleParent.Condition != COND_Never)
		{
			if (COND_None != RemoteRoleParent.Condition)
			{
				UE_LOGF(LogRep, Warning, "FRepLayout::InitFromClass: Forcing replication of RemoteRole. Owner=%ls", *InObjectClass->GetPathName());
			}

			Parents[(int32)AActor::ENetFields_Private::RemoteRole].Flags |= ERepParentFlags::IsConditional;
			Parents[(int32)AActor::ENetFields_Private::RemoteRole].Condition = COND_None;
		}
	}	

#if WITH_PUSH_MODEL
	if (bIsPushModelEnabled && ((NumberOfLifetimePushModelProperties > 0) || (NumberOfFastArrayPushModelProperties > 0)))
	{
		const bool bFullPushProperties = (NumberOfLifetimeProperties == NumberOfLifetimePushModelProperties);

		if (bFullPushProperties)
		{
			Flags |= ERepLayoutFlags::FullPushProperties;
		}

		Flags |= (bFullPushProperties && (NumberOfFastArrayProperties == NumberOfFastArrayPushModelProperties)) ?
			ERepLayoutFlags::FullPushSupport :
			ERepLayoutFlags::PartialPushSupport;
	}
#endif

	if (NumberOfLifetimeProperties == 0 && !LifetimeCustomPropertyState.IsValid())
	{
		Flags |= ERepLayoutFlags::NoReplicatedProperties;
	}

	if (!ServerConnection || EnumHasAnyFlags(CreateFlags, ECreateRepLayoutFlags::MaySendProperties))
	{
		BuildHandleToCmdIndexTable_r(0, Cmds.Num() - 1, BaseHandleToCmdIndex);
	}

	BuildShadowOffsets<ERepBuildType::Class>(InObjectClass, Parents, Cmds, ShadowDataBufferSize);

	Owner = InObjectClass;
}

TSharedPtr<FRepLayout> FRepLayout::CreateFromFunction(UFunction* InFunction, const UNetConnection* ServerConnection, const ECreateRepLayoutFlags CreateFlags)
{
	TSharedPtr<FRepLayout> RepLayout = MakeShareable<FRepLayout>(new FRepLayout());
	RepLayout->InitFromFunction(InFunction, ServerConnection, CreateFlags);
	return RepLayout;
}

void FRepLayout::InitFromFunction(
	UFunction* InFunction,
	const UNetConnection* ServerConnection,
	const ECreateRepLayoutFlags CreateFlags)
{
	int32 RelativeHandle = 0;

	for (TFieldIterator<FProperty> It(InFunction); It && (It->PropertyFlags & (CPF_Parm | CPF_ReturnParm)) == CPF_Parm; ++It)
	{
		for (int32 ArrayIdx = 0; ArrayIdx < It->ArrayDim; ++ArrayIdx)
		{
			const int32 ParentHandle = AddParentProperty(Parents, *It, ArrayIdx);

			FInitFromPropertySharedParams SharedParams
			{
				/*Cmds=*/Cmds,
				/*ServerConnection=*/ServerConnection,
				/*ParentIndex=*/ParentHandle,
				/*Parent=*/Parents[ParentHandle],
			};

			FInitFromPropertyStackParams StackParams
			{
				/*Property=*/*It,
				/*Offset=*/It->GetElementSize()* ArrayIdx,
				/*RelativeHandle=*/RelativeHandle,
				/*ParentChecksum=*/0,
				/*StaticArrayIndex=*/ArrayIdx
			};

			Parents[ParentHandle].CmdStart = Cmds.Num();
			RelativeHandle = InitFromProperty_r<ERepBuildType::Function>(SharedParams, StackParams);
			Parents[ParentHandle].CmdEnd = Cmds.Num();
			Parents[ParentHandle].Offset = GetOffsetForProperty<ERepBuildType::Function>(**It);

			SetupRepStructFlags(Parents[ParentHandle], /**bSkipCustomDeltaCheck=*/true);
		}
	}

	AddReturnCmd(Cmds);

	if (!ServerConnection || EnumHasAnyFlags(CreateFlags, ECreateRepLayoutFlags::MaySendProperties))
	{
		BuildHandleToCmdIndexTable_r(0, Cmds.Num() - 1, BaseHandleToCmdIndex);
	}

	BuildShadowOffsets<ERepBuildType::Function>(InFunction, Parents, Cmds, ShadowDataBufferSize);

	Owner = InFunction;
}

TSharedPtr<FRepLayout> FRepLayout::CreateFromStruct(
	UStruct* InStruct,
	const UNetConnection* ServerConnection,
	const ECreateRepLayoutFlags CreateFlags)
{
	TSharedPtr<FRepLayout> RepLayout = MakeShareable<FRepLayout>(new FRepLayout());
	RepLayout->InitFromStruct(InStruct, ServerConnection, CreateFlags);
	return RepLayout;
}

void FRepLayout::InitFromStruct(
	UStruct* InStruct,
	const UNetConnection* ServerConnection,
	const ECreateRepLayoutFlags CreateFlags)
{
	int32 RelativeHandle = 0;

	for (TFieldIterator<FProperty> It(InStruct); It; ++It)
	{
		if (It->PropertyFlags & CPF_RepSkip)
		{
			continue;
		}
			
		for (int32 ArrayIdx = 0; ArrayIdx < It->ArrayDim; ++ArrayIdx)
		{
			const int32 ParentHandle = AddParentProperty(Parents, *It, ArrayIdx);

			FInitFromPropertySharedParams SharedParams
			{
				/*Cmds=*/Cmds,
				/*ServerConnection=*/ServerConnection,
				/*ParentIndex=*/ParentHandle,
				/*Parent=*/Parents[ParentHandle],
			};

			FInitFromPropertyStackParams StackParams
			{
				/*Property=*/*It,
				/*Offset=*/It->GetElementSize() * ArrayIdx,
				/*RelativeHandle=*/RelativeHandle,
				/*ParentChecksum=*/0,
				/*StaticArrayIndex=*/ArrayIdx
			};

			Parents[ParentHandle].CmdStart = Cmds.Num();
			RelativeHandle = InitFromProperty_r<ERepBuildType::Struct>(SharedParams, StackParams);
			Parents[ParentHandle].CmdEnd = Cmds.Num();
			Parents[ParentHandle].Offset = GetOffsetForProperty<ERepBuildType::Struct>(**It);

			SetupRepStructFlags(Parents[ParentHandle], /**bSkipCustomDeltaCheck=*/true);
		}
	}

	AddReturnCmd(Cmds);

	if (!ServerConnection || EnumHasAnyFlags(CreateFlags, ECreateRepLayoutFlags::MaySendProperties))
	{
		BuildHandleToCmdIndexTable_r(0, Cmds.Num() - 1, BaseHandleToCmdIndex);
	}

	BuildShadowOffsets<ERepBuildType::Struct>(InStruct, Parents, Cmds, ShadowDataBufferSize);

	Owner = InStruct;
}

void FRepLayout::SerializeProperties_DynamicArray_r(
	FBitArchive& Ar, 
	UPackageMap* Map,
	const int32 CmdIndex,
	FRepObjectDataBuffer Data,
	bool& bHasUnmapped,
	const int32 ArrayDepth,
	const FRepSerializationSharedInfo& SharedInfo,
	FNetTraceCollector* Collector,
	const UObject* OwningObject) const
{
	const FRepLayoutCmd& Cmd = Cmds[ CmdIndex ];
	if (EnumHasAnyFlags(Cmd.Flags, ERepLayoutCmdFlags::IsEmptyArrayStruct))
	{
		return;
	}

	FScriptArray* Array = (FScriptArray*)Data.Data;

	uint16 OutArrayNum = Array->Num();
	Ar << OutArrayNum;

	// If loading from the archive, OutArrayNum will contain the number of elements.
	// Otherwise, use the input number of elements.
	const int32 ArrayNum = Ar.IsLoading() ? (int32)OutArrayNum : Array->Num();

	// Validate the maximum number of elements.
	if (!UE_RepLayout_Private::ValidateArraySize(ArrayNum, Cmd.Property))
	{
		Ar.SetError();
	}

	if (!Ar.IsError())
	{
		// When loading, we may need to resize the array to properly fit the number of elements.
		if (Ar.IsLoading() && OutArrayNum != Array->Num())
		{
			FScriptArrayHelper ArrayHelper((FArrayProperty*)Cmd.Property, Data);
			ArrayHelper.Resize(OutArrayNum);
		}

		FRepObjectDataBuffer ArrayData(Array->GetData());

		for (int32 i = 0; i < Array->Num() && !Ar.IsError(); i++)
		{
			const int32 ArrayElementOffset = i * Cmd.ElementSize;
			SerializeProperties_r(Ar, Map, CmdIndex + 1, Cmd.EndCmd - 1, ArrayData + ArrayElementOffset, bHasUnmapped, i, ArrayDepth, SharedInfo, Collector, OwningObject);
		}
	}	
}

void FRepLayout::SerializeProperties_r(
	FBitArchive& Ar, 
	UPackageMap* Map,
	const int32 CmdStart, 
	const int32 CmdEnd,
	FRepObjectDataBuffer Data,
	bool& bHasUnmapped,
	const int32 ArrayIndex,
	const int32 ArrayDepth,
	const FRepSerializationSharedInfo& SharedInfo,
	FNetTraceCollector* Collector,
	const UObject* OwningObject) const
{
	for (int32 CmdIndex = CmdStart; CmdIndex < CmdEnd && !Ar.IsError(); CmdIndex++)
	{
		const FRepLayoutCmd& Cmd = Cmds[CmdIndex];

		check(Cmd.Type != ERepLayoutCmdType::Return);

		if (Cmd.Type == ERepLayoutCmdType::DynamicArray)
		{
			UE_NET_TRACE_DYNAMIC_NAME_SCOPE(Cmd.Property->GetFName(), Ar, Collector, ENetTraceVerbosity::Trace);

			SerializeProperties_DynamicArray_r(Ar, Map, CmdIndex, Data + Cmd, bHasUnmapped, ArrayDepth + 1, SharedInfo, Collector, OwningObject);
			CmdIndex = Cmd.EndCmd - 1;		// The -1 to handle the ++ in the for loop
			continue;
		}

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
		if (GDoReplicationContextString > 0 && Map)
		{
			Map->SetDebugContextString(FString::Printf(TEXT("%s - %s"), *Owner->GetPathName(), *Cmd.Property->GetPathName()));
		}
#endif

		const FRepSerializedPropertyInfo* SharedPropInfo = nullptr;

		if ((GNetSharedSerializedData != 0) && Ar.IsSaving() && EnumHasAnyFlags(Cmd.Flags, ERepLayoutCmdFlags::IsSharedSerialization))
		{
			FRepSharedPropertyKey PropertyKey(CmdIndex, ArrayIndex, ArrayDepth, (void*)(Data + Cmd).Data);

			SharedPropInfo = SharedInfo.SharedPropertyInfo.FindByPredicate([PropertyKey](const FRepSerializedPropertyInfo& Info) 
			{ 
				return (Info.PropertyKey == PropertyKey);
			});
		}

		if (Ar.IsLoading() && Map)
		{
			Map->ResetTrackedSyncLoadedGuids();
		}

		// Use shared serialization state if it exists
		// Not concerned with unmapped guids because object references can't be shared
		if (SharedPropInfo)
		{
			check(SharedInfo.SerializedProperties.IsValid());

			GNumSharedSerializationHit++;
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
			if ((GNetVerifyShareSerializedData != 0) && Ar.IsSaving())
			{
				// TODO: Is this actually a NetBitWriter?
				FBitWriter& Writer = static_cast<FBitWriter&>(Ar);

				FBitWriterMark BitWriterMark(Writer);

				Cmd.Property->NetSerializeItem(Writer, Map, (Data + Cmd).Data);

				TArray<uint8> StandardBuffer;
				BitWriterMark.Copy(Writer, StandardBuffer);
				BitWriterMark.Pop(Writer);

				Writer.SerializeBitsWithOffset(SharedInfo.SerializedProperties->GetData(), SharedPropInfo->PropBitOffset, SharedPropInfo->PropBitLength);

				TArray<uint8> SharedBuffer;
				BitWriterMark.Copy(Writer, SharedBuffer);

				if (StandardBuffer != SharedBuffer)
				{
					UE_LOGF(LogRep, Error, "Shared serialization data mismatch!");
				}
			}
			else
#endif
			{
				Ar.SerializeBitsWithOffset(SharedInfo.SerializedProperties->GetData(), SharedPropInfo->PropBitOffset, SharedPropInfo->PropBitLength);
			}
		}
		else
		{
			UE_NET_TRACE_DYNAMIC_NAME_SCOPE(Cmd.Property->GetFName(), Ar, Collector, ENetTraceVerbosity::Trace);

			GNumSharedSerializationMiss++;
			if (!Cmd.Property->NetSerializeItem(Ar, Map, (Data + Cmd).Data))
			{
				bHasUnmapped = true;
			}
		}

		if (Ar.IsLoading() && Map)
		{
			Map->ReportSyncLoadsForProperty(Cmd.Property, OwningObject);
		}

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
		if (GDoReplicationContextString > 0 && Map)
		{
			Map->ClearDebugContextString();
		}
#endif
	}
}

void FRepLayout::BuildChangeList_r(
	const TArray<FHandleToCmdIndex>& HandleToCmdIndex,
	const int32 CmdStart,
	const int32 CmdEnd,
	const FConstRepObjectDataBuffer Data,
	const int32 HandleOffset,
	const bool bForceArraySends,
	TArray<uint16>& Changed) const
{
	for (int32 CmdIndex = CmdStart; CmdIndex < CmdEnd; CmdIndex++)
	{
		const FRepLayoutCmd& Cmd = Cmds[CmdIndex];

		check(Cmd.Type != ERepLayoutCmdType::Return);

		if (Cmd.Type == ERepLayoutCmdType::DynamicArray)
		{			
			FScriptArray* Array = (FScriptArray *)(Data + Cmd).Data;
			const FConstRepObjectDataBuffer ArrayData = Array->GetData();

			TArray<uint16> ChangedLocal;

			TArray<FHandleToCmdIndex>& ArrayHandleToCmdIndex = *HandleToCmdIndex[Cmd.RelativeHandle - 1].HandleToCmdIndex;

			const int32 ArrayCmdStart = CmdIndex + 1;
			const int32 ArrayCmdEnd = Cmd.EndCmd - 1;
			const int32 NumHandlesPerElement = ArrayHandleToCmdIndex.Num();

			check(NumHandlesPerElement > 0);

			for (int32 i = 0; i < Array->Num(); i++)
			{
				const int32 ArrayElementOffset = Cmd.ElementSize * i;
				BuildChangeList_r(ArrayHandleToCmdIndex, ArrayCmdStart, ArrayCmdEnd, ArrayData + ArrayElementOffset, i * NumHandlesPerElement, bForceArraySends, ChangedLocal);
			}

			if (ChangedLocal.Num())
			{
				Changed.Add(Cmd.RelativeHandle + HandleOffset);	// Identify the array cmd handle
				Changed.Add(ChangedLocal.Num());				// This is so we can jump over the array if we need to
				Changed.Append(ChangedLocal);					// Append the change list under the array
				Changed.Add(0);									// Null terminator
			}
			else if (bForceArraySends)
			{
				// Note the handle, that there were 0 changed elements, and the terminator.
				// This will force anyone sending this changelist later to send the array size
				// (which is most likely 0 in this case).
				Changed.Add(Cmd.RelativeHandle + HandleOffset);
				Changed.Add(0);
				Changed.Add(0);
			}

			CmdIndex = Cmd.EndCmd - 1;		// The -1 to handle the ++ in the for loop
			continue;
		}

		Changed.Add(Cmd.RelativeHandle + HandleOffset);
	}
}


void FRepLayout::BuildSharedSerialization(
	const FConstRepObjectDataBuffer Data,
	TArray<uint16>& Changed,
	const bool bWriteHandle,
	FRepSerializationSharedInfo& SharedInfo,
	UE::Net::FNetTokenStore* NetTokenStore) const
{
#ifdef ENABLE_PROPERTY_CHECKSUMS
	const bool bDoChecksum = (GDoPropertyChecksum == 1);
#else
	const bool bDoChecksum = false;
#endif

	FChangelistIterator ChangelistIterator(Changed, 0);
	FRepHandleIterator HandleIterator(Owner, ChangelistIterator, Cmds, BaseHandleToCmdIndex, 0, 1, 0, Cmds.Num() - 1);

	SharedInfo.Init();

	// Create scope to export NetTokens, storing pending exports in SharedInfoRPC.NetTokensPendingExport
	UE::Net::FNetTokenExportScope NetTokenExportScope(*SharedInfo.SerializedProperties, NetTokenStore, SharedInfo.NetTokensPendingExport, "BuildSharedSerialization");

	BuildSharedSerialization_r(HandleIterator, Data, bWriteHandle, bDoChecksum, 0, SharedInfo);

	SharedInfo.SetValid();
}

void FRepLayout::BuildSharedSerialization_r(
	FRepHandleIterator& HandleIterator,
	const FConstRepObjectDataBuffer SourceData,
	const bool bWriteHandle,
	const bool bDoChecksum,
	const int32 ArrayDepth,
	FRepSerializationSharedInfo& SharedInfo) const
{
	while (HandleIterator.NextHandle())
	{
		const int32 CmdIndex = HandleIterator.CmdIndex;
		const int32 ArrayOffset = HandleIterator.ArrayOffset;

		const FRepLayoutCmd& Cmd = Cmds[CmdIndex];
		const FRepParentCmd& ParentCmd = Parents[Cmd.ParentIndex];

		const FConstRepObjectDataBuffer Data = SourceData + ArrayOffset + Cmd;

		// Custom Deltas are not supported for shared serialization at this time.
		if (EnumHasAnyFlags(ParentCmd.Flags, ERepParentFlags::IsCustomDelta))
		{
			continue;
		}

		if (Cmd.Type == ERepLayoutCmdType::DynamicArray)
		{
			const FScriptArray* Array = (FScriptArray *)Data.Data;
			const FConstRepObjectDataBuffer ArrayData = (uint8*)Array->GetData();

			FScopedIteratorArrayTracker ArrayTracker(&HandleIterator);

			TArray<FHandleToCmdIndex>& ArrayHandleToCmdIndex = *HandleIterator.HandleToCmdIndex[Cmd.RelativeHandle - 1].HandleToCmdIndex;

			FRepHandleIterator ArrayIterator(HandleIterator.Owner, HandleIterator.ChangelistIterator, Cmds, ArrayHandleToCmdIndex, Cmd.ElementSize, Array->Num(), CmdIndex + 1, Cmd.EndCmd - 1);
			BuildSharedSerialization_r(ArrayIterator, ArrayData, bWriteHandle, bDoChecksum, ArrayDepth + 1, SharedInfo);
			continue;
		}

		if (EnumHasAnyFlags(Cmd.Flags, ERepLayoutCmdFlags::IsSharedSerialization))
		{
			FRepSharedPropertyKey PropertyKey(HandleIterator.CmdIndex, HandleIterator.ArrayIndex, ArrayDepth, (void*)Data.Data);

			SharedInfo.WriteSharedProperty(Cmd, PropertyKey, HandleIterator.CmdIndex, HandleIterator.Handle, Data.Data, bWriteHandle, bDoChecksum);
		}
	}
}

void FRepLayout::BuildSharedSerializationForRPC_DynamicArray_r(
	const int32 CmdIndex,
	const FConstRepObjectDataBuffer Data,
	int32 ArrayDepth,
	FRepSerializationSharedInfo& SharedInfo)
{
	const FRepLayoutCmd& Cmd = Cmds[CmdIndex];
	if (EnumHasAnyFlags(Cmd.Flags, ERepLayoutCmdFlags::IsEmptyArrayStruct))
	{
		return;
	}

	FScriptArray* Array = (FScriptArray *)Data.Data;	
	const int32 ArrayNum = Array->Num();

	if (!UE_RepLayout_Private::ValidateArraySize(ArrayNum, Cmd.Property))
	{
		return;
	}

	const FConstRepObjectDataBuffer ArrayData(Array->GetData());

	for (int32 i = 0; i < ArrayNum; i++)
	{
		const int32 ArrayElementOffset = i * Cmd.ElementSize;
		BuildSharedSerializationForRPC_r(CmdIndex + 1, Cmd.EndCmd - 1, ArrayData + ArrayElementOffset, i, ArrayDepth, SharedInfo);
	}
}

void FRepLayout::BuildSharedSerializationForRPC_r(
	const int32 CmdStart,
	const int32 CmdEnd,
	const FConstRepObjectDataBuffer Data,
	int32 ArrayIndex,
	int32 ArrayDepth,
	FRepSerializationSharedInfo& SharedInfo)
{
	for (int32 CmdIndex = CmdStart; CmdIndex < CmdEnd; CmdIndex++)
	{
		const FRepLayoutCmd& Cmd = Cmds[CmdIndex];

		check(Cmd.Type != ERepLayoutCmdType::Return);

		if (Cmd.Type == ERepLayoutCmdType::DynamicArray)
		{
			BuildSharedSerializationForRPC_DynamicArray_r(CmdIndex, Data + Cmd, ArrayDepth + 1, SharedInfo);
			CmdIndex = Cmd.EndCmd - 1;		// The -1 to handle the ++ in the for loop
			continue;
		}

		if (!Parents[Cmd.ParentIndex].Property->HasAnyPropertyFlags(CPF_OutParm) && EnumHasAnyFlags(Cmd.Flags, ERepLayoutCmdFlags::IsSharedSerialization))
		{
			FRepSharedPropertyKey PropertyKey(CmdIndex, ArrayIndex, ArrayDepth, (void*)(Data + Cmd).Data);

			SharedInfo.WriteSharedProperty(Cmd, PropertyKey, CmdIndex, 0, (Data + Cmd).Data, false, false);
		}
	}
}

void FRepLayout::BuildSharedSerializationForRPC(const FConstRepObjectDataBuffer Data, UE::Net::FNetTokenStore* NetTokenStore)
{
	if ((GNetSharedSerializedData != 0) && !SharedInfoRPC.IsValid())
	{
		SharedInfoRPC.Init();
		SharedInfoRPCParentsChanged.Init(false, Parents.Num());

		// Create scope to export NetTokens, storing pending exports in SharedInfoRPC.NetTokensPendingExport
		UE::Net::FNetTokenExportScope ExportScope(*SharedInfoRPC.SerializedProperties, NetTokenStore, SharedInfoRPC.NetTokensPendingExport, "BuildSharedSerializationForRPC");

		for (int32 i = 0; i < Parents.Num(); i++)
		{
			if (Parents[i].Property->HasAnyPropertyFlags(CPF_OutParm))
			{
				continue;
			}

			bool bSend = true;

			if (!CastField<FBoolProperty>(Parents[i].Property))
			{
				// check for a complete match, including arrays
				// (we're comparing against zero data here, since 
				// that's the default.)
				bSend = !Parents[i].Property->Identical_InContainer(Data.Data, nullptr, Parents[i].ArrayIndex);
			}

			if (bSend)
			{
				// Cache result of property comparison to default so we only have to do it once
				SharedInfoRPCParentsChanged[i] = true;

				BuildSharedSerializationForRPC_r(Parents[i].CmdStart, Parents[i].CmdEnd, Data, 0, 0, SharedInfoRPC);
			}
		}

		SharedInfoRPC.SetValid();
	}
}

void FRepLayout::ClearSharedSerializationForRPC()
{
	SharedInfoRPC.Reset();
	SharedInfoRPCParentsChanged.Reset();
}

void FRepLayout::SendPropertiesForRPC(
	UFunction* Function,
	UActorChannel* Channel,
	FNetBitWriter& Writer,
	const FConstRepObjectDataBuffer Data) const
{
	check(Function == Owner);

	if (!IsEmpty())
	{
		if (Channel->Connection->IsInternalAck())
		{
			TArray<uint16> Changed;

			for (int32 i = 0; i < Parents.Num(); i++)
			{
				if (!Parents[i].Property->Identical_InContainer(Data.Data, NULL, Parents[i].ArrayIndex))
				{
					BuildChangeList_r(BaseHandleToCmdIndex, Parents[i].CmdStart, Parents[i].CmdEnd, Data, 0, false, Changed);
				}
			}

			Changed.Add(0); // Null terminator

			SendProperties_BackwardsCompatible(nullptr, nullptr, Data, Channel->Connection, Writer, Changed);
		}
		else
		{
			bool bAppendNetTokenExports = false;
			for (int32 i = 0; i < Parents.Num(); i++)
			{
				bool Send = true;

				if (!CastField<FBoolProperty>(Parents[i].Property))
				{
					// Used cached comparison result if possible
					if ((GNetSharedSerializedData != 0) && SharedInfoRPC.IsValid() && !Parents[i].Property->HasAnyPropertyFlags(CPF_OutParm))
					{
						Send = SharedInfoRPCParentsChanged[i];
					}
					else
					{
						// check for a complete match, including arrays
						// (we're comparing against zero data here, since 
						// that's the default.)
						Send = !Parents[i].Property->Identical_InContainer(Data.Data, NULL, Parents[i].ArrayIndex);
					}

					Writer.WriteBit(Send ? 1 : 0);
				}

				if (Send)
				{
					bAppendNetTokenExports = true;
					bool bHasUnmapped = false;
					SerializeProperties_r(Writer, Writer.PackageMap, Parents[i].CmdStart, Parents[i].CmdEnd, const_cast<uint8*>(Data.Data), bHasUnmapped, 0, 0, SharedInfoRPC, GetTraceCollector(Writer), nullptr);
				}
			}
			// Append potential exports from shared serialization			
			if (UE::Net::FNetTokenExportContext* NetTokenExportContext = bAppendNetTokenExports && SharedInfoRPC.IsValid() ? Writer.NetTokenExportContext.Get() : nullptr)
			{
				NetTokenExportContext->AppendNetTokensPendingExport(SharedInfoRPC.NetTokensPendingExport);
			}
		}	
	}
}

void FRepLayout::ReceivePropertiesForRPC(
	UObject* Object,
	UFunction* Function,
	UActorChannel* Channel,
	FNetBitReader& Reader,
	FRepObjectDataBuffer Data,
	TSet<FNetworkGUID>&	UnmappedGuids) const
{
	check(Function == Owner);

	if (!IsEmpty())
	{
		for (int32 i = 0; i < Parents.Num(); i++)
		{
			if (Parents[i].ArrayIndex == 0 && !EnumHasAnyFlags(Parents[i].Flags, ERepParentFlags::IsZeroConstructible))
			{
				// If this property needs to be constructed, make sure we do that
				Parents[i].Property->InitializeValue(Data + Parents[i]);
			}
		}

		if (Channel->Connection->IsInternalAck())
		{
			bool bHasUnmapped = false;
			bool bGuidsChanged = false;

			// Let package map know we want to track and know about any guids that are unmapped during the serialize call
			// We have to do this manually since we aren't passing in any unmapped info
			Reader.PackageMap->ResetTrackedGuids(true);

			ReceiveProperties_BackwardsCompatible(Channel->Connection, nullptr, Data, Reader, bHasUnmapped, false, bGuidsChanged, Object);

			if (Reader.PackageMap->GetTrackedUnmappedGuids().Num() > 0)
			{
				bHasUnmapped = true;
				UnmappedGuids = Reader.PackageMap->GetTrackedUnmappedGuids();
			}

			Reader.PackageMap->ResetTrackedGuids(false);

			if (bHasUnmapped)
			{
				UE_LOGF(LogRepTraffic, Log, "Unable to resolve RPC parameter to do being unmapped. Object[%d] %ls. Function %ls.",
					Channel->ChIndex, *Object->GetName(), *Function->GetName());
			}
		}
		else
		{
			Reader.PackageMap->ResetTrackedGuids(true);

			static FRepSerializationSharedInfo Empty;
			FNetTraceCollector* Collector = Channel->Connection->GetInTraceCollector();
	
			for (int32 i = 0; i < Parents.Num(); i++)
			{
				if (CastField<FBoolProperty>(Parents[i].Property) || Reader.ReadBit())
				{
					bool bHasUnmapped = false;

					SerializeProperties_r(Reader, Reader.PackageMap, Parents[i].CmdStart, Parents[i].CmdEnd, Data, bHasUnmapped, 0, 0, Empty, Collector, Object);

					if (Reader.IsError())
					{
						return;
					}

					if (bHasUnmapped)
					{
						UE_LOGF(LogRepTraffic, Log, "Unable to resolve RPC parameter. Object[%d] %ls. Function %ls. Parameter %ls.",
							Channel->ChIndex, *Object->GetName(), *Function->GetName(), *Parents[i].Property->GetName());
					}
				}
			}

			if (Reader.PackageMap->GetTrackedUnmappedGuids().Num() > 0)
			{
				UnmappedGuids = Reader.PackageMap->GetTrackedUnmappedGuids();
			}

			Reader.PackageMap->ResetTrackedGuids(false);
		}
	}
}

void FRepLayout::SerializePropertiesForStruct(
	UStruct* Struct,
	FBitArchive& Ar,
	UPackageMap* Map,
	FRepObjectDataBuffer Data,
	bool& bHasUnmapped,
	const UObject* OwningObject) const
{
	check(Struct == Owner);

	static FRepSerializationSharedInfo Empty;

	for (int32 i = 0; i < Parents.Num(); i++)
	{
		SerializeProperties_r(Ar, Map, Parents[i].CmdStart, Parents[i].CmdEnd, Data, bHasUnmapped, 0, 0, Empty, nullptr, OwningObject);

		if (Ar.IsError())
		{
			return;
		}
	}
}

void FRepLayout::BuildHandleToCmdIndexTable_r(
	const int32 CmdStart,
	const int32 CmdEnd,
	TArray<FHandleToCmdIndex>& HandleToCmdIndex)
{
	for (int32 CmdIndex = CmdStart; CmdIndex < CmdEnd; CmdIndex++)
	{
		const FRepLayoutCmd& Cmd = Cmds[CmdIndex];

		check(Cmd.Type != ERepLayoutCmdType::Return);

		const int32 Index = HandleToCmdIndex.Add(FHandleToCmdIndex(CmdIndex));

		if (Cmd.Type == ERepLayoutCmdType::DynamicArray)
		{
			HandleToCmdIndex[Index].HandleToCmdIndex = TUniquePtr<TArray<FHandleToCmdIndex>>(new TArray<FHandleToCmdIndex>());

			TArray<FHandleToCmdIndex>& ArrayHandleToCmdIndex = *HandleToCmdIndex[Index].HandleToCmdIndex;

			BuildHandleToCmdIndexTable_r(CmdIndex + 1, Cmd.EndCmd - 1, ArrayHandleToCmdIndex);
			CmdIndex = Cmd.EndCmd - 1;		// The -1 to handle the ++ in the for loop
		}
	}
}

bool FSendingRepState::HasAnyPendingRetirements() const
{
	for (const FPropertyRetirement& PropRet : Retirement)
	{
		if (PropRet.Next != nullptr)
		{
			return true;
		}
	}

	return false;
}

void FRepLayout::RebuildConditionalProperties(FSendingRepState* RepState, const FReplicationFlags RepFlags) const
{
	SCOPE_CYCLE_COUNTER(STAT_NetRebuildConditionalTime);
	
	RepState->RepFlags = RepFlags;

	TStaticBitArray<COND_Max> ConditionMap = UE::Net::BuildConditionMapFromRepFlags(RepFlags);
	if (EnumHasAnyFlags(Flags, ERepLayoutFlags::HasDynamicConditionProperties) && RepState->RepChangedPropertyTracker.IsValid())
	{
		const FRepChangedPropertyTracker* RepChangedPropertyTracker = RepState->RepChangedPropertyTracker.Get();
		for (auto It = TBitArray<>::FIterator(RepState->InactiveParents); It; ++It)
		{
			ELifetimeCondition Condition = Parents[It.GetIndex()].Condition;
			if (Condition == COND_Dynamic)
			{
				Condition = RepChangedPropertyTracker->GetDynamicCondition(It.GetIndex());
			}
			It.GetValue() = !ConditionMap[Condition];
		}
	}
	else
	{
		for (auto It = TBitArray<>::FIterator(RepState->InactiveParents); It; ++It)
		{
			It.GetValue() = !ConditionMap[Parents[It.GetIndex()].Condition];
		}
	}
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
void FRepLayout::InitChangedTracker(FRepChangedPropertyTracker* ChangedTracker) const
{
	checkf(ChangedTracker->GetParentCount() == Parents.Num(), TEXT("InitChangedTracker: Mismatched replicated parent properties for: %s"), *GetNameSafe(Owner));
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

FRepStateStaticBuffer FRepLayout::CreateShadowBuffer(const FConstRepObjectDataBuffer Source) const
{
	FRepStateStaticBuffer ShadowData(AsShared());

	if (!IsEmpty())
	{
		if (ShadowDataBufferSize == 0)
		{
			UE_LOGF(LogRep, Error, "FRepLayout::InitShadowData: Invalid RepLayout: %ls", *GetPathNameSafe(Owner));
		}
		else
		{
			InitRepStateStaticBuffer(ShadowData, Source);
		}
	}

	return ShadowData;
}

TSharedPtr<FReplicationChangelistMgr> FRepLayout::CreateReplicationChangelistMgr(const UObject* InObject, const ECreateReplicationChangelistMgrFlags CreateFlags) const
{
	// ChangelistManager / ChangelistState will hold onto a unique pointer for this
	// so no need to worry about deleting it here.

	FCustomDeltaChangelistState* DeltaChangelistState = nullptr;
	if (!EnumHasAnyFlags(CreateFlags, ECreateReplicationChangelistMgrFlags::SkipDeltaCustomState) && LifetimeCustomPropertyState && !!LifetimeCustomPropertyState->GetNumFastArrayProperties())
	{
		DeltaChangelistState = new FCustomDeltaChangelistState(LifetimeCustomPropertyState->GetNumFastArrayProperties());
	}

	const uint8* ShadowStateSource = (const uint8*)InObject->GetArchetype();
	if (ShadowStateSource == nullptr)
	{
		UE_LOGF(LogRep, Error, "FRepLayout::CreateReplicationChangelistMgr: Invalid object archetype, initializing shadow state to current object state: %ls", *GetFullNameSafe(InObject));
		ShadowStateSource = (const uint8*)InObject;
	}

	return MakeShareable(new FReplicationChangelistMgr(AsShared(), ShadowStateSource, InObject, DeltaChangelistState));
}

TUniquePtr<FRepState> FRepLayout::CreateRepState(
	const FConstRepObjectDataBuffer Source,
	TSharedPtr<FRepChangedPropertyTracker>& InRepChangedPropertyTracker,
	ECreateRepStateFlags CreateFlags) const
{
	// TODO: We could probably avoid allocating a RepState completely if we detect the RepLayout is empty.
	//		In that case, there won't be anything interesting to do anyway.
	//		This would require more sanity checks in code using RepStates though.

	TUniquePtr<FRepState> RepState(new FRepState());

	// If we have a changelist manager, that implies we're acting as a server.
	const bool bIsServer = InRepChangedPropertyTracker.IsValid();

	// In that case, we don't need to initialize the shadow data, as it
	// will be stored in the ChangelistManager for this object once for all connections.
	if (InRepChangedPropertyTracker.IsValid())
	{
		check(InRepChangedPropertyTracker->GetParentCount() == Parents.Num());

		RepState->SendingRepState.Reset(new FSendingRepState());
		RepState->SendingRepState->RepChangedPropertyTracker = InRepChangedPropertyTracker;

		// Start out the conditional props based on a default RepFlags struct
		// It will rebuild if it ever changes
		RebuildConditionalProperties(RepState->SendingRepState.Get(), FReplicationFlags());
		RepState->SendingRepState->InactiveParents.Init(false, Parents.Num());
	}
	
	if (!EnumHasAnyFlags(CreateFlags, ECreateRepStateFlags::SkipCreateReceivingState))
	{
		FRepStateStaticBuffer StaticBuffer(AsShared());

		// For server's, we don't need ShadowData as the ChangelistTracker / Manager will be used
		// instead.
		if (!bIsServer)
		{
			InitRepStateStaticBuffer(StaticBuffer, Source);
		}

		RepState->ReceivingRepState.Reset(new FReceivingRepState(MoveTemp(StaticBuffer)));
	}

	return RepState;
}

void FRepLayout::InitRepStateStaticBuffer(FRepStateStaticBuffer& ShadowData, const FConstRepObjectDataBuffer Source) const
{
	check(ShadowData.Buffer.Num() == 0);
	ShadowData.Buffer.SetNumZeroed(ShadowDataBufferSize);
	ConstructProperties(ShadowData);
	CopyProperties(ShadowData, Source);
}

void FRepLayout::ConstructProperties(FRepStateStaticBuffer& InShadowData) const
{
	FRepShadowDataBuffer ShadowData = InShadowData.GetData();

	// Construct all items
	for (const FRepParentCmd& Parent : Parents)
	{
		// Only construct the 0th element of static arrays (InitializeValue will handle the elements)
		if (Parent.ArrayIndex == 0)
		{
			check((Parent.ShadowOffset + Parent.Property->GetSize()) <= InShadowData.Num());
			Parent.Property->InitializeValue(ShadowData + Parent);
		}
	}
}

void FRepLayout::CopyProperties(FRepStateStaticBuffer& InShadowData, const FConstRepObjectDataBuffer Source) const
{
	FRepShadowDataBuffer ShadowData = InShadowData.GetData();

	// Init all items
	for (const FRepParentCmd& Parent : Parents)
	{
		// Only copy the 0th element of static arrays (CopyCompleteValue will handle the elements)
		if (Parent.ArrayIndex == 0)
		{
			check((Parent.ShadowOffset + Parent.Property->GetSize()) <= InShadowData.Num());
			Parent.Property->CopyCompleteValue(ShadowData + Parent, Source + Parent);
		}
	}
}

void FRepLayout::DestructProperties(FRepStateStaticBuffer& InShadowData) const
{
	FRepShadowDataBuffer ShadowData = InShadowData.GetData();

	// Destruct all items
	for (const FRepParentCmd& Parent : Parents)
	{
		// Only destroy the 0th element of static arrays (DestroyValue will handle the elements)
		if (Parent.ArrayIndex == 0)
		{
			check((Parent.ShadowOffset + Parent.Property->GetSize()) <= InShadowData.Num());
			Parent.Property->DestroyValue(ShadowData + Parent);
		}
	}

	InShadowData.Buffer.Empty();
}

void FRepLayout::AddReferencedObjects(FReferenceCollector& Collector)
{
	FProperty* Current = nullptr;
	for (FRepParentCmd& Parent : Parents)
	{
		Current = Parent.Property;
		if (Current != nullptr)
		{
			Current->AddReferencedObjects(Collector);

			// The only way this could happen is if a property was marked pending kill.
			// Technically, that could happen for a BP Property if its class is no longer needed,
			// but that should also clean up the FRepLayout.
			if (Current == nullptr)
			{
				UE_LOGF(LogRep, Error, "Replicated Property is no longer valid: %ls",  *(Parent.CachedPropertyName.ToString()));
				Parent.Property = nullptr;
			}
		}
	}
}

FString FRepLayout::GetReferencerName() const
{
	return TEXT("FRepLayout");
}

// TODO: There's a better way to do this, but it requires more changes.
//			Ideally, we bring Retirements management, etc., into RepLayout.
//			What we could do instead of using standard changelists is use individual circular buffers 
//			for any given array element. Each time we see a new Fast Array Rep ID, we'd add a new element
//			to the buffer. Once all connections have acked that history, we would remove it from the buffer.
//			We could still enforce a hard limit to the size of the buffers.
//
//			We might technically be able to do that without bringing retirement stuff in,
//			but it would require us exposing the handles to NetSerialization.h (where fast TArrays live)
//			and that gets included all over the place, and it'd be better to not.


// It's important to note that unlike normal RepLayout properties which require changelists to know
// what changed and am accumulated "lifetime changelist" to catch up late joiners or people with bad
// connections, Fast Arrays don't need that.
//
// Instead, Fast Arrays track Rep IDs for each individual element, and for the array itself.
// Initially, all keys start as 0 and are incremented as the items are marked dirty.
// If we ever receive a NAK, we revert our IDs back to the last ACKed ID state.
//
// This means that at any given time, each connection knows what its current state is, and
// can just compare that against the "real state" stored on the Fast Array.
//
// In FastArrayDeltaSerialize (and the DeltaSerializeStructs variant), we use those IDs to determine
// what items actually need to be sent to a client.
//
// We also store the Changelist History ID alongside the Array / Element Rep IDs, and that is reset in the
// case of NAKs and starts at a designated invalid state initially.
//
// Changelists used here are purely accelerations. In ideal scenarios, we do property
// comparisons once per frame and store those changelists, just like normal rep layouts.
// Also like normal RepLayouts, that will be shared across all connections.
//
// When we go to replicate a Fast Array, we just accumulate the changelists for all of its
// dirty items, and then send those.
//
// On initial send, or if a given connection receives too many NAKs and falls outside of our History Range,
// we will devolve into sending the full state of the dirty items. At that point we're guaranteed that
// the connection is up to date (if it receives a NAK on that packet, its state will be reset back to
// being outside history range, and we try again).

void FRepLayout::PreSendCustomDeltaProperties(
	UObject* Object,
	UNetConnection* Connection,
	FReplicationChangelistMgr& ChangelistMgr,
	uint32 ReplicationFrame,
	TArray<TSharedPtr<INetDeltaBaseState>>& CustomDeltaStates) const
{
	using namespace UE_RepLayout_Private;

	if (!Connection->IsInternalAck())
	{
		const FLifetimeCustomDeltaState& LocalLifetimeCustomPropertyState = *LifetimeCustomPropertyState;

		if (LocalLifetimeCustomPropertyState.GetNumFastArrayProperties())
		{
			FRepChangelistState& ChangelistState = *ChangelistMgr.GetRepChangelistState();
			FCustomDeltaChangelistState& CustomDeltaChangelistState = *ChangelistState.CustomDeltaChangelistState;

			// Check to see whether or not we need to do comparisons this frame.
			// If we do, then run through our fast array states and generate new history items if needed.
			if (CustomDeltaChangelistState.LastReplicationFrame != ReplicationFrame)
			{
				CustomDeltaChangelistState.LastReplicationFrame = ReplicationFrame;

				const FConstRepObjectDataBuffer ObjectData(Object);
				const uint16 NumLifetimeCustomDeltaProperties = LocalLifetimeCustomPropertyState.GetNumCustomDeltaProperties();

				for (uint16 CustomDeltaIndex = 0; CustomDeltaIndex < NumLifetimeCustomDeltaProperties; ++CustomDeltaIndex)
				{
					const FLifetimeCustomDeltaProperty& CustomDeltaProperty = LocalLifetimeCustomPropertyState.GetCustomDeltaProperty(CustomDeltaIndex);
					const uint16 RepIndex = CustomDeltaProperty.PropertyRepIndex;

					// If our Fast Array Items Command is invalid, we can't do anything.
					// This should have been logged on RepLayout creation.
					if (INDEX_NONE != CustomDeltaProperty.FastArrayItemsCommand)
					{
						const int32 FastArrayNumber = CustomDeltaProperty.FastArrayNumber;
						const FRepParentCmd& FastArrayCmd = Parents[RepIndex];

						void const * const FastArraySerializer = ObjectData + FastArrayCmd;
						const EFastArraySerializerDeltaFlags DeltaFlags = CustomDeltaProperty.GetFastArrayDeltaFlags(FastArraySerializer);

						// Note, we can't rely on EFastArraySerializerDeltaFlags::HasBeenSerialized here.
						// It's possible we're calling PreSendCustomDeltaProperties **before** the first time the struct
						// was ever serialized, and in that case it would still be false, and prevent us from creating
						// a history the first time.
						//
						// This does mean that Fast Arrays requesting delta serialization will still have their history
						// incremented the first time, even if the feature is generally disabled.
						//
						// TODO: If any fast arrays failed this check, we could probably reset their state,
						//			because we know we should never try sending them again
						if (EnumHasAnyFlags(DeltaFlags, EFastArraySerializerDeltaFlags::IsUsingDeltaSerialization) ||
							(!EnumHasAnyFlags(DeltaFlags, EFastArraySerializerDeltaFlags::HasBeenSerialized) &&
								EnumHasAnyFlags(DeltaFlags, EFastArraySerializerDeltaFlags::HasDeltaBeenRequested)))
						{
							FDeltaArrayHistoryState& FastArrayHistoryState = CustomDeltaChangelistState.ArrayStates[FastArrayNumber];

							// If the fast array's ReplicationKey hasn't changed, then we can safely assume there's been no changes.
							const int32 FastArrayReplicationKey = CustomDeltaProperty.GetFastArrayArrayReplicationKey(FastArraySerializer);
							if (FastArrayHistoryState.ArrayReplicationKey != FastArrayReplicationKey)
							{
								FastArrayHistoryState.InitHistory();

								const uint32 HistoryDelta = FastArrayHistoryState.HistoryEnd - FastArrayHistoryState.HistoryStart;
								const uint32 CurrentHistoryIndex = FastArrayHistoryState.HistoryEnd % FDeltaArrayHistoryState::MAX_CHANGE_HISTORY;
								const FDeltaArrayHistoryItem& CurrentHistory = FastArrayHistoryState.ChangeHistory[CurrentHistoryIndex];
								const bool bCurrentHistoryUpdated = FastArrayHistoryState.ChangeHistoryUpdated[CurrentHistoryIndex];

								// If we don't have any history items, go ahead and create one.
								// Otherwise, check to see if our current history was actually updated.
								// If it wasn't updated, that means that no one tried to replicate it last frame (which can be possible due
								// to rep conditions), and there's no sense in creating a new one.
								if (HistoryDelta == 0 || bCurrentHistoryUpdated)
								{
									// If we've reached our buffer size, then move our start history marker up.
									// In that case the old start history will become our new history.
									if (HistoryDelta >= FDeltaArrayHistoryState::MAX_CHANGE_HISTORY)
									{
										++FastArrayHistoryState.HistoryStart;
									}

									++FastArrayHistoryState.HistoryEnd;
									const uint32 NewHistory = FastArrayHistoryState.HistoryEnd % FDeltaArrayHistoryState::MAX_CHANGE_HISTORY;
									FastArrayHistoryState.ChangeHistory[NewHistory].Reset();
									FastArrayHistoryState.ChangeHistoryUpdated[NewHistory] = false;
								}
							}
						}
					}
				}
			}
		}
	}
}

void FRepLayout::PostSendCustomDeltaProperties(
	UObject* Object,
	UNetConnection* Connection,
	FReplicationChangelistMgr& ChangelistMgr,
	TArray<TSharedPtr<INetDeltaBaseState>>& CustomDeltaStates) const
{
}

ERepLayoutResult FRepLayout::DeltaSerializeFastArrayProperty(FFastArrayDeltaSerializeParams& Params, FReplicationChangelistMgr* ChangelistMgr) const
{
	using namespace UE_RepLayout_Private;

	CONDITIONAL_SCOPE_CYCLE_COUNTER(STAT_RepLayout_DeltaSerializeFastArray, GUseDetailedScopeCounters);

	// A portion of this work could be shared across all Fast Array Properties for a given object,
	// but that would be easier to do if the Custom Delta Serialization was completely encapsulated in FRepLayout.

	check(LifetimeCustomPropertyState);

	FNetDeltaSerializeInfo& DeltaSerializeInfo = Params.DeltaSerializeInfo;

	const FLifetimeCustomDeltaProperty& CustomDeltaProperty = LifetimeCustomPropertyState->GetCustomDeltaProperty(DeltaSerializeInfo.CustomDeltaIndex);
	const uint16 ParentIndex = CustomDeltaProperty.PropertyRepIndex;

	const FRepParentCmd& Parent = Parents[ParentIndex];
	const int32 CmdIndex = CustomDeltaProperty.FastArrayItemsCommand;

	if (INDEX_NONE == CmdIndex)
	{
		// This should have already been caught by InitFromClass.
		// So, log with a lower verbosity.
		UE_LOGF(LogRep, Log, "FRepLayout::DeltaSerializeFastArrayProperty: Invalid fast array items command index! %ls", *Parent.CachedPropertyName.ToString())
		return ERepLayoutResult::Error;
	}

	const FRepLayoutCmd& FastArrayItemCmd = Cmds[CmdIndex];
	const int32 ElementSize = FastArrayItemCmd.ElementSize;

	const int32 ItemLayoutStart = CmdIndex + 1;
	const int32 ItemLayoutEnd = FastArrayItemCmd.EndCmd - 1;

	UObject* Object = DeltaSerializeInfo.Object;
	UPackageMapClient* PackageMap = static_cast<UPackageMapClient*>(DeltaSerializeInfo.Map);
	UNetConnection* Connection = DeltaSerializeInfo.Connection;
	const bool bIsWriting = !!DeltaSerializeInfo.Writer;
	const bool bInternalAck = Connection->IsInternalAck();

	FRepObjectDataBuffer ObjectData(Object);
	FScriptArray* ObjectArray = GetTypedProperty<FScriptArray>(ObjectData, FastArrayItemCmd);
	FRepObjectDataBuffer ObjectArrayData(ObjectArray->GetData());

	FFastArraySerializer& ArraySerializer = Params.ArraySerializer;

	check(&ArraySerializer == &Params.ArraySerializer);

	const int32 ObjectArrayNum = ObjectArray->Num();
	FNetFieldExportGroup* NetFieldExportGroup = nullptr;

	if (bInternalAck)
	{
		// Note, PackageMap should hold onto the strong reference for us, so we use raw pointers where
		// we can.

		// TODO: This feels like something we could cache in PreSend, but we'd need to add plumbing to hold onto it.
		const FString OwnerPathName = Owner->GetPathName();
		TSharedPtr<FNetFieldExportGroup> LocalNetFieldExportGroup = PackageMap->GetNetFieldExportGroup(OwnerPathName);

		if (!LocalNetFieldExportGroup.IsValid())
		{
			if (!bIsWriting)
			{
				UE_LOGF(LogRep, Error, "DeltaSerializeFastArrayProperty: Unable to find NetFieldExportGroup during replay playback. Class=%ls, Property=%ls", *Owner->GetName(), *Parent.CachedPropertyName.ToString());
				return ERepLayoutResult::Error;
			}

			UE_LOGF(LogRepProperties, VeryVerbose, "DeltaSerializeFastArrayProperty: Create Netfield Export Group.")
			LocalNetFieldExportGroup = CreateNetfieldExportGroup();
			PackageMap->AddNetFieldExportGroup(OwnerPathName, LocalNetFieldExportGroup);
		}

		NetFieldExportGroup = LocalNetFieldExportGroup.Get();
	}

	if (bIsWriting)
	{
		FNetBitWriter& Writer = static_cast<FNetBitWriter&>(*DeltaSerializeInfo.Writer);
		auto& ChangedElements = *Params.WriteChangedElements;
		
		// This is a list of changelists to send, corresponding to items in ChangedElements.
		TArray<TArray<uint16>> Changelists;

		const FHandleToCmdIndex& ArrayHandleToCmd = BaseHandleToCmdIndex[FastArrayItemCmd.RelativeHandle - 1];
		const TArray<FHandleToCmdIndex>& ArrayHandleToCmdIndex = *ArrayHandleToCmd.HandleToCmdIndex;

		// We only need to create changelists if we're not in a demo.
		// Note, we may change this in the future to also delta structs for replays.
		if (!bInternalAck)
		{
			// First, we'll create our changelists.
			if (ChangelistMgr)
			{
				FRepChangelistState& RepChangelistState = *ChangelistMgr->GetRepChangelistState();
				FCustomDeltaChangelistState& DeltaChangelistState = *RepChangelistState.CustomDeltaChangelistState;

				const int32 FastArrayNumber = CustomDeltaProperty.FastArrayNumber;
				FDeltaArrayHistoryState& FastArrayState = DeltaChangelistState.ArrayStates[FastArrayNumber];

				// Params.WriteBaseState should be valid, and have the most up to date IDToChangelist map for the Fast Array.
				// However, it's ChangelistHistory will be to the last History Number sent to the Fast TArray on the specific
				// connection we're replicating from.
				TSharedRef<FNetFastTArrayBaseState> NewArrayDeltaState = StaticCastSharedRef<FNetFastTArrayBaseState>(Params.WriteBaseState->AsShared());

				// Cache off the newest history, the last history sent to this connection, and then update the state
				// to notify that we're going to send it the newest history.
				const uint32 NewChangelistHistory = FastArrayState.HistoryEnd;
				const uint32 LastAckedHistory = NewArrayDeltaState->GetLastAckedHistory();
				const uint32 LastAckedChangelistDelta = NewChangelistHistory - LastAckedHistory;

				NewArrayDeltaState->SetChangelistHistory(NewChangelistHistory);

				// Cache off the shadow array buffers.
				FRepShadowDataBuffer ShadowData(RepChangelistState.StaticBuffer.GetData());
				FScriptArray* ShadowArray = GetTypedProperty<FScriptArray>(ShadowData, FastArrayItemCmd);
				FRepShadowDataBuffer ShadowArrayData(ShadowArray->GetData());

				// Note, we explicitly pass in a 0 handles everywhere below.
				// This is because each item will be received individually, the receiving side indices won't
				// necessarily match our indices, and we already track the changelists separately.

				// Check to see whether or not we need to update the global changelist shared between connections.
				{
					const uint32 RelativeNewHistory = NewChangelistHistory % FDeltaArrayHistoryState::MAX_CHANGE_HISTORY;
					const uint32 CompareChangelistDelta = NewChangelistHistory - FastArrayState.HistoryStart;
					FDeltaArrayHistoryItem& HistoryItem = FastArrayState.ChangeHistory[RelativeNewHistory];
					const bool bHistoryItemUpdated = FastArrayState.ChangeHistoryUpdated[RelativeNewHistory];

					if (!bHistoryItemUpdated)
					{
						FastArrayState.ChangeHistoryUpdated[RelativeNewHistory] = true;

						FastArrayState.ArrayReplicationKey = NewArrayDeltaState->ArrayReplicationKey;

						// Update our shadow array, and reset our pointer in case we reallocated.
						FScriptArrayHelper ShadowArrayHelper((FArrayProperty*)FastArrayItemCmd.Property, ShadowArray);

						TBitArray<> ShadowArrayItemIsNew(false, ObjectArrayNum);
						const bool bIsInitial = CompareChangelistDelta == 1;

						// It's possible that elements have been deleted or otherwise reordered, and our shadow state is out of date.
						// In order to prevent issues, we'll shuffle our shadow state back to the correct order.
						// Note, we can't just do a lookup in both maps directly below, because we might end up stomping the same
						// shadow state multiple times.
						//
						//	Conceptually, you can imagine this process as having two lines of elements: the Object Line and the Shadow Line.
						//	The Object Line is always considered authoritative, and we just need to make sure the Shadow Line matches that.
						//
						//	If the Shadow Line is empty, the only thing we need to do is add a matching number of elements
						//	and patch their IDs to match those of the Object Line.
						//
						//	If the Object Line is empty, the only thing we need to do is remove all elements from the shadow line.
						//
						//	If both lines are non empty, it becomes a fairly straightforward fixup, following these rules.
						//	It's important to note, that at each step all previously seen elements are guaranteed to have been
						//	validated, and are generally considered "out of play" or no longer in the line, and because of
						//	that we should never touch them again.
						//
						//	The process stops when either we run out of Shadow Line or Object Line elements.
						//
						//		1. If Elements at the front of the line have matching IDs, there's nothing that needs to be done
						//			and we can move onto the next Element in both lines.
						//
						//		2. If Elements at the front of the line have mismatched IDs, then it's either because the Object Element
						//			was reordered, a previous Object Element was deleted, or a new Object Element was added.
						//
						//			a. If we find the Shadow Element in the Shadow Line, it must have been reordered.
						//				Go ahead and swap the current front of the Shadow Line with the found Shadow Element.
						//				Now, the elements at the front of the line have matching IDs, and we can move onto
						//				the next Element in both lines.
						//
						//			b. If we don't find the Shadow Element in the Shadow Line, the Object Element must be new.
						//				Go ahead and insert a new item into the Shadow array, fix up its ID, and mark it as new.
						//				Now, the elements at the front of the line have matching IDs, and we can move onto
						//				the next Element in both lines.
						//
						//		3. If there are elements remaining in both Lines, go back to step 1. Otherwise, continue to step 4.
						//
						//		4. At this point, there should be 3 possible outcomes:
						//
						//			a. The Object Line and the Shadow Line have the same number of Elements, and all are matching. We're done.
						//
						//			b. The Object Line has more elements than the Shadow Line. All missing elements from the Shadow Line
						//				must be new elements.
						//
						//			c. The Shadow Line has more elements than the Object Line. All missing elements from the Object Line
						//				must have been removed.

						// TODO: Optimize this. Luckily, it only happens once per frame, and only if the array is dirty.
						//			Maybe this could be merged into the sending code below, the only concern is that
						//			doesn't tracked deleted elements.
						//
						//			Alternatively, if Custom Delta code was merged into FRepLayout, we might be able to track
						//			lists of deleted items on a given frame and merge those together just like changelists.
						//			This would prevent us from needing to call BuildChangedAndDeletedBuffers on Fast TArrays
						//			for every connection, unless a specific connection was very out of date.

						// Note, this code serves a very similar purpose to FFastArraySerializer::TFastArraySerializeHelper<Type, SerializerType>::BuildChangedAndDeletedBuffers.
						// The main issue is that we can't rely on that method, because it will be comparing the last state that was replicated to a given particular connection,
						// and we want to compare the last state that was replicated to *any* connection.

						{
							FScriptArrayHelper ObjectArrayHelper((FArrayProperty*)FastArrayItemCmd.Property, ObjectArray);

							// We track this as a non-const, because if we append any items into the middle of the
							// array, they will be explicitly marked as new, and we still want to compare items
							// that existed in the array originally.
							int32 ShadowArrayNum = ShadowArrayHelper.Num();

							if (ObjectArrayNum != 0 && ShadowArrayNum != 0 && !bIsInitial)
							{
								const TMap<int32, int32> OldShadowIDToIndexMap(MoveTemp(FastArrayState.IDToIndexMap));
								FastArrayState.IDToIndexMap.Reserve(ObjectArrayNum);

								// We track the Appended Shadow Items, because any index we try and use after such
								// an append needs to be shifted appropriately.
								// TODO: We may be able to iterate the list backwards instead, but that may break
								//			some assumptions laid out in the algorithm above.
								int32 AppendedShadowItems = 0;

								UE_LOGF(LogRepProperties, VeryVerbose, "DeltaSerializeFastArrayProperty: Fixup Shadow State. Owner=%ls, Object=%ls, Property=%ls, bInitial=%d, ObjectArrayNum=%d, ShadowArrayNum=%d",
									*Owner->GetName(), *Object->GetPathName(), *Parent.CachedPropertyName.ToString(), !!bIsInitial, ObjectArrayNum, ShadowArrayHelper.Num());

								for (int32 Index = 0; Index < ObjectArrayNum && Index < ShadowArrayNum; ++Index)
								{
									const int32 ObjectReplicationID = CustomDeltaProperty.GetFastArrayItemReplicationID(ObjectArrayHelper.GetRawPtr(Index));
									int32& ShadowReplicationID = CustomDeltaProperty.GetFastArrayItemReplicationIDMutable(ShadowArrayHelper.GetRawPtr(Index));

									FastArrayState.IDToIndexMap.Emplace(ObjectReplicationID, Index);

									UE_LOGF(LogRepProperties, VeryVerbose, "DeltaSerializeFastArrayProperty: Handling Item. ID=%d, Index=%d, ShadowID=%d", ObjectReplicationID, Index, ShadowReplicationID);

									// If our IDs match, there's nothing to do.
									if (ObjectReplicationID != ShadowReplicationID)
									{
										// The IDs didn't match, so this is an insert, delete, or swap.
										if (int32 const * const FoundShadowIndex = OldShadowIDToIndexMap.Find(ObjectReplicationID))
										{
											// We found the element in the shadow array, so there must have been a swap.
											// Sanity check that the invalid element can only possibly later in our lines.
											const int32 FixedShadowIndex = *FoundShadowIndex + AppendedShadowItems;

											UE_LOGF(LogRepProperties, VeryVerbose, "DeltaSerializeFastArrayProperty: Swapped Shadow Item. OldIndex=%d, NewIndex=%d", Index, FixedShadowIndex);

											check(FixedShadowIndex > Index);

											ShadowArrayHelper.SwapValues(Index, FixedShadowIndex);
										}
										else
										{
											// This item must have been inserted into the array (or appended and then shuffled in).
											// So, insert it into our shadow array and update its ID.

											ShadowArrayItemIsNew[Index] = true;
											ShadowArrayHelper.InsertValues(Index);

											int32& NewShadowReplicationID = CustomDeltaProperty.GetFastArrayItemReplicationIDMutable(ShadowArrayHelper.GetRawPtr(Index));
											NewShadowReplicationID = ObjectReplicationID;

											++AppendedShadowItems;
											++ShadowArrayNum;
											UE_LOGF(LogRepProperties, VeryVerbose, "DeltaSerializeFastArrayProperty: Added Shadow Item. AppendedShadowItems=%d", AppendedShadowItems);
										}
									}
								}
							}

							// Now we can go ahead and resize the array, to make any other changes we need.
							ShadowArrayHelper.Resize(ObjectArrayNum);
							ShadowArrayData = ShadowArray->GetData();

							// Go ahead and fix up IDs for any elements that may have just been appended.
							// Note, we need to do this for all elements on the initial pass.
							// Deleted elements will have been chopped off by the resize.
							if (bIsInitial || (ShadowArrayNum < ObjectArrayNum))
							{
								UE_CLOGF(bIsInitial, LogRepProperties, VeryVerbose, "DeltaSerializeFastArrayProperty: Adding initial properties. Owner=%ls, Object=%ls, Property=%ls, bInitial=%d, ObjectArrayNum=%d, ShadowArrayNum=%d",
									*Owner->GetName(), *Object->GetPathName(), *Parent.CachedPropertyName.ToString(), !!bIsInitial, ObjectArrayNum, ShadowArrayHelper.Num());

								const int32 StartIndex = bIsInitial ? 0 : ShadowArrayNum;
								for (int32 Index = StartIndex; Index < ObjectArrayNum; ++Index)
								{
									const int32 ObjectReplicationID = CustomDeltaProperty.GetFastArrayItemReplicationID(ObjectArrayHelper.GetRawPtr(Index));
									int32& ShadowReplicationID = CustomDeltaProperty.GetFastArrayItemReplicationIDMutable(ShadowArrayHelper.GetRawPtr(Index));

									ShadowReplicationID = ObjectReplicationID;
									ShadowArrayItemIsNew[Index] = true;

									UE_LOGF(LogRep, VeryVerbose, "DeltaSerializeFastArrayProperty: Added Shadow Item. Index=%d, ID=%d", Index, ShadowReplicationID);
									FastArrayState.IDToIndexMap.Emplace(ObjectReplicationID, Index);
								}
							}
						}

						TArray<uint16> NewChangelist;
						for (auto& IDIndexPair : ChangedElements)
						{
							NewChangelist.Empty(1);
							const int32 ArrayElementOffset = ElementSize * IDIndexPair.Idx;

							const bool bForceFail = !UE::Net::Private::bDeltaInitialFastArrayElements && (bIsInitial || ShadowArrayItemIsNew[IDIndexPair.Idx]);

							// Go ahead and do a property compare here, regardless of what we'll actually use below.
							// This is to prevent issues where someone with an initial / outdated connection doesn't properly
							// update the changelists in our history, but does update the shadow state inadvertently.
							FComparePropertiesSharedParams SharedParams{
								/*bIsInitial=*/ bIsInitial,
								/*bForceFail=*/ bForceFail,
								Flags,
								Parents,
								Cmds,
								nullptr,
								nullptr,
								nullptr,
								NetSerializeLayouts
							};

							ERepLayoutResult UpdateResult = ERepLayoutResult::Success;

							FComparePropertiesStackParams StackParams{
								FConstRepObjectDataBuffer(ObjectArrayData + ArrayElementOffset),
								FRepShadowDataBuffer(ShadowArrayData + ArrayElementOffset),
								NewChangelist,
								UpdateResult
							};

							CompareProperties_r(SharedParams, StackParams, ItemLayoutStart, ItemLayoutEnd, 0);

							// NOTE: Currently, SA always throws "Warning: Expression 'ERepLayoutResult::FatalError == UpdateResult' is always false" here.
							//			It gets tripped up by the indirection / reference semantics of Stack Params and assumes that
							//			result just remains the same value as it was assigned above.
							//			Because of this, the error is disabled.
							if (UNLIKELY(ERepLayoutResult::FatalError == UpdateResult)) // -V547
							{
								return UpdateResult;
							}

							if (NewChangelist.Num())
							{
								NewChangelist.Add(0);
								HistoryItem.ChangelistByID.Emplace(IDIndexPair.ID, MoveTemp(NewChangelist));

								// If our FastArraySerializerItems are NetSerialize, then their ID may be reset to INDEX_NONE due
								// to copying them into the shadow state (see FFastArraySerializerItem::operator=).
								// In that case, we need to make make sure we reset our ID so they can be found the next
								// time we try to replicate them.
								int32& ShadowReplicationID = CustomDeltaProperty.GetFastArrayItemReplicationIDMutable(StackParams.ShadowData.Data);
								ShadowReplicationID = IDIndexPair.ID;
							}
						}
					}
				}

				// Now, merge all of the changelists we need together.
				// If we're sufficiently far back, or if this is our first transmission, then we'll just force fail
				// and send all changes (happens in the block below, Changelists.Num() == 0).

				// Note, this won't be all changes since the beginning, but just all changes for the currently dirty items.
				const uint32 LastHistory = LastAckedHistory;
				const uint32 LastChangelistDelta = LastAckedChangelistDelta;

				const bool bAllowInitialHistory = UE::Net::Private::bDeltaInitialFastArrayElements || (LastHistory != 0);

				if (bAllowInitialHistory && LastChangelistDelta > 0 && LastChangelistDelta < (FDeltaArrayHistoryState::MAX_CHANGE_HISTORY - 1))
				{
					const FConstRepObjectDataBuffer ConstObjectData(ObjectData);
					Changelists.SetNum(ChangedElements.Num());

					// Note, we iterate from LastAckedHistory + 1, because we don't want to send something if
					// we think it's already been sent/received.
					// Similarly, we do <= NewChangelistHistory because we need to send the newest history.
					for (uint32 ChangelistHistory = LastHistory + 1; ChangelistHistory <= NewChangelistHistory; ++ChangelistHistory)
					{
						const uint32 RelativeHistory = ChangelistHistory % FDeltaArrayHistoryState::MAX_CHANGE_HISTORY;
						FDeltaArrayHistoryItem& HistoryItem = FastArrayState.ChangeHistory[RelativeHistory];

						for (int32 i = 0; i < ChangedElements.Num(); ++i)
						{
							const auto& IDIndexPair = ChangedElements[i];
							if (const TArray<uint16>* FoundChangelist = HistoryItem.ChangelistByID.Find(IDIndexPair.ID))
							{
								if (FoundChangelist->Num() > 1)
								{
									// This is basically the DynamicArray case from MergeChangelists, but specialized.
									// We could probably just make that more generic.
									// Might also be worth creating a helper Lambda / Struct to create ChangelistIterators / HandleIterators.

									TArray<uint16>& ElementChangelist = Changelists[i];
									TArray<uint16> Temp = MoveTemp(ElementChangelist);
									ElementChangelist.Empty(1);

									const FConstRepObjectDataBuffer ElementData(ObjectArrayData + (IDIndexPair.Idx * ElementSize));

									FChangelistIterator FoundChangelistIterator(*FoundChangelist, 0);
									FRepHandleIterator FoundHandleIterator(Owner, FoundChangelistIterator, Cmds, ArrayHandleToCmdIndex, ElementSize, 1, ItemLayoutStart, ItemLayoutEnd);

									if (Temp.Num() == 0)
									{
										PruneChangeList_r(FoundHandleIterator, ElementData, ElementChangelist);
									}
									else
									{
										FChangelistIterator ElementChangelistIterator(Temp, 0);
										FRepHandleIterator ElementHandleIterator(Owner, ElementChangelistIterator, Cmds, ArrayHandleToCmdIndex, ElementSize, 1, ItemLayoutStart, ItemLayoutEnd);

										MergeChangeList_r(FoundHandleIterator, ElementHandleIterator, ElementData, ElementChangelist);
									}

									ElementChangelist.Add(0);
								}
							}
						}
					}
				}
			}

			if (Changelists.Num() == 0)
			{
				// If we didn't end up building changelists earlier for whatever reason, go ahead and just
				// build a full changelist for all changed elements.

				// This could have happened if we were sending initially, we were outside of history range, or
				// we didn't have a changelist manager.

				Changelists.SetNum(ChangedElements.Num());
				for (int32 i = 0; i < ChangedElements.Num(); ++i)
				{
					const auto& IDIndexPair = ChangedElements[i];
					const int32 ArrayElementOffset = ElementSize * IDIndexPair.Idx;

					FConstRepObjectDataBuffer ElementData(ObjectArrayData + ArrayElementOffset);

					TArray<uint16>& Changelist = Changelists[i];
					BuildChangeList_r(ArrayHandleToCmdIndex, ItemLayoutStart, ItemLayoutEnd, ElementData, 0, true, Changelist);

					if (Changelist.Num())
					{
						Changelist.Add(0);
					}
				}
			}
		}

		// Ignore tracking properties in Network Profiler below.
		// We will rely on the normal custom delta property tracking which happens elsewhere.
		NETWORK_PROFILER_IGNORE_PROPERTY_SCOPE

		// Now that we have our changelists setup, we can send the data.
		for (int32 i = 0; i < ChangedElements.Num(); ++i)
		{
			UE_NET_TRACE_SCOPE(ChangedElement, Writer, GetTraceCollector(Writer), ENetTraceVerbosity::Trace);

			const auto& IDIndexPair = ChangedElements[i];
			uint32 ID = ChangedElements[i].ID;
			Writer << ID;

			const int32 ArrayElementOffset = ElementSize * IDIndexPair.Idx;
			FConstRepObjectDataBuffer ElementData(ObjectArrayData + ArrayElementOffset);

			if (bInternalAck)
			{
				SendAllProperties_BackwardsCompatible_r(
					nullptr,
					Writer,
					false,
					PackageMap,
					NetFieldExportGroup,
					ItemLayoutStart,
					ItemLayoutEnd,
					ElementData);
			}
			else
			{
				TArray<uint16>& Changelist = Changelists[i];
				const bool bAnythingToSend = Changelist.Num() > 1;
				Writer.WriteBit(!!bAnythingToSend);

				if (bAnythingToSend)
				{
					FChangelistIterator ChangelistIterator(Changelist, 0);
					FRepHandleIterator HandleIterator(
						Owner,
						ChangelistIterator,
						Cmds,
						ArrayHandleToCmdIndex,
						ElementSize,
						1,
						ItemLayoutStart,
						ItemLayoutEnd);

					SendProperties_r(
						/*RepState=*/ nullptr,
						Writer,
						/*bDoChecksum=*/ false,
						HandleIterator,
						ElementData,
						/*ArrayDepth=*/ 1,
						/*SharedInfo=*/ nullptr,
						ESerializePropertyType::Handle);

					WritePropertyHandle(Writer, 0, false);
				}
			}
			
		}

		return !Writer.IsError() ? ERepLayoutResult::Success : ERepLayoutResult::Error;
	}
	else
	{
		FNetBitReader& Reader = static_cast<FNetBitReader&>(*DeltaSerializeInfo.Reader);
		TArray<int32, TInlineAllocator<8>>& ChangedElements = *Params.ReadChangedElements;
		TArray<int32, TInlineAllocator<8>>& AddedElements = *Params.ReadAddedElements;

		FScriptArrayHelper FastArrayHelper((FArrayProperty*)FastArrayItemCmd.Property, ObjectArray);

		bool bOutGuidsChanged = false;
		bool bOutHasUnmapped = false;

		// WARNING! Don't attempt to use ObjectArrayData below, always rely on FastArrayHelper.
		// The helper may reallocate the array, and invalidate that pointer.

		for (int32 i = 0; i < Params.ReadNumChanged; ++i)
		{
			uint32 ID = 0;
			Reader << ID;

			int32* ElementIndexPtr = ArraySerializer.ItemMap.Find(ID);
			int32 ElementIndex = 0;
			void* ThisElement = nullptr;

			if (!ElementIndexPtr)
			{
				UE_LOGF(LogNetFastTArray, Log, "   New. ID: %d. New Element!", ID);

				ElementIndex = FastArrayHelper.AddValue();
				ArraySerializer.ItemMap.Add(ID, ElementIndex);
				AddedElements.Add(ElementIndex);
			}
			else
			{
				ElementIndex = *ElementIndexPtr;
				ChangedElements.Add(ElementIndex);

				UE_LOGF(LogNetFastTArray, Log, "   Changed. ID: %d -> Idx: %d", ID, ElementIndex);
			}

			ThisElement = FastArrayHelper.GetRawPtr(ElementIndex);

			Params.ReceivedItem(ThisElement, Params, ID);

			FGuidReferencesMap& GuidReferences = ArraySerializer.GuidReferencesMap_StructDelta.FindOrAdd(ID);

			if (bInternalAck)
			{
				const bool bSuccess = ReceiveProperties_BackwardsCompatible_r(
					nullptr,
					NetFieldExportGroup,
					Reader,
					ItemLayoutStart,
					ItemLayoutEnd,
					nullptr,
					ThisElement,
					ThisElement,
					&GuidReferences,
					bOutHasUnmapped,
					bOutGuidsChanged,
					Object);

				if (!bSuccess)
				{
					UE_LOGF(LogNetFastTArray, Warning, "FRepLayout::DeltaSerializeFastArrayProperty: Failed to receive backwards compat properties!");
					return ERepLayoutResult::Error;
				}
			}
			else
			{
				const bool bAnythingSent = !!Reader.ReadBit();
				if (!bAnythingSent)
				{
					continue;
				}

				FReceivePropertiesSharedParams SharedParams{
					/*bDoChecksum=*/ false,
					/*bSkipRoleSwap=*/ !EnumHasAnyFlags(Flags, ERepLayoutFlags::IsActor),
					Reader,
					bOutHasUnmapped,
					bOutGuidsChanged,
					Parents,
					Cmds,
					NetSerializeLayouts,
					Object
				};

				FReceivePropertiesStackParams StackParams{
					ThisElement,
					nullptr,
					&GuidReferences,
					ItemLayoutStart,
					ItemLayoutEnd,
					/*RepNotifies=*/ nullptr
				};

				// Read the first handle, and then start receiving properties.
				ReadPropertyHandle(SharedParams);
				if (ReceiveProperties_r(SharedParams, StackParams))
				{
					if (0 != SharedParams.ReadHandle)
					{
						UE_LOGF(LogRep, Error, "ReceiveFastArrayItem: Invalid property terminator handle - Handle=%d", SharedParams.ReadHandle);
						return ERepLayoutResult::Error;
					}
				}
				else
				{
					UE_LOGF(LogNetFastTArray, Warning, "FRepLayout::DeltaSerializeFastArrayProperty: Failed to received properties");
					return ERepLayoutResult::Error;
				}
			}

			if (Reader.IsError())
			{
				UE_LOGF(LogNetFastTArray, Warning, "FRepLayout::DeltaSerializeFastArrayProperty: Reader.IsError() == true");
				return ERepLayoutResult::Error;
			}

			DeltaSerializeInfo.bGuidListsChanged |= bOutGuidsChanged;
			DeltaSerializeInfo.bOutHasMoreUnmapped |= bOutHasUnmapped;
		}

		return ERepLayoutResult::Success;
	}
}

void FRepLayout::GatherGuidReferencesForFastArray(FFastArrayDeltaSerializeParams& Params) const
{
	using namespace UE_RepLayout_Private;

	const FConstRepObjectDataBuffer ObjectData(Params.DeltaSerializeInfo.Object);
	const FLifetimeCustomDeltaProperty& CustomDeltaProperty = LifetimeCustomPropertyState->GetCustomDeltaProperty(Params.DeltaSerializeInfo.CustomDeltaIndex);
	const FRepParentCmd& Parent = Parents[CustomDeltaProperty.PropertyRepIndex];

	const FFastArraySerializer& ArraySerializer = Params.ArraySerializer;
	TSet<FNetworkGUID>& GatherGuids = *Params.DeltaSerializeInfo.GatherGuidReferences;
	int32 TrackedGuidMemory = 0;

	for (auto& GuidReferencesPair : ArraySerializer.GuidReferencesMap_StructDelta)
	{
		GatherGuidReferences_r(&GuidReferencesPair.Value, GatherGuids, TrackedGuidMemory);
	}

	if (Params.DeltaSerializeInfo.TrackedGuidMemoryBytes)
	{
		*Params.DeltaSerializeInfo.TrackedGuidMemoryBytes += TrackedGuidMemory;
	}
}

bool FRepLayout::MoveMappedObjectToUnmappedForFastArray(FFastArrayDeltaSerializeParams& Params) const
{
	using namespace UE_RepLayout_Private;

	const FRepObjectDataBuffer ObjectData(Params.DeltaSerializeInfo.Object);
	const FLifetimeCustomDeltaProperty& CustomDeltaProperty = LifetimeCustomPropertyState->GetCustomDeltaProperty(Params.DeltaSerializeInfo.CustomDeltaIndex);
	const FRepParentCmd& Parent = Parents[CustomDeltaProperty.PropertyRepIndex];

	FFastArraySerializer& ArraySerializer = Params.ArraySerializer;
	const FNetworkGUID& MoveToUnmapped = *Params.DeltaSerializeInfo.MoveGuidToUnmapped;

	bool bFound = false;
	for (auto& GuidReferencesPair : ArraySerializer.GuidReferencesMap_StructDelta)
	{
		bFound |= MoveMappedObjectToUnmapped_r(&GuidReferencesPair.Value, MoveToUnmapped, Params.DeltaSerializeInfo.Object);
	}
	return bFound;
}

void FRepLayout::UpdateUnmappedGuidsForFastArray(FFastArrayDeltaSerializeParams& Params) const
 {
	using namespace UE_RepLayout_Private;

	check(LifetimeCustomPropertyState);

	FNetDeltaSerializeInfo& DeltaSerializeInfo = Params.DeltaSerializeInfo;

	const FLifetimeCustomDeltaProperty& CustomDeltaProperty = LifetimeCustomPropertyState->GetCustomDeltaProperty(DeltaSerializeInfo.CustomDeltaIndex);
	const int32 ParentIndex = CustomDeltaProperty.PropertyRepIndex;
	const FRepParentCmd& Parent = Parents[ParentIndex];
	const int32 CmdIndex = CustomDeltaProperty.FastArrayItemsCommand;
	
	if (INDEX_NONE == CmdIndex)
	{
		// This should have already been caught by InitFromClass.
		// So, log with a lower verbosity.
		UE_LOGF(LogRep, Log, "FRepLayout::UpdateUnmappedGuidsForFastArray: Invalid fast array items command index! %ls", *Parent.CachedPropertyName.ToString())
		return;
	}

	const FRepLayoutCmd& FastArrayItemCmd = Cmds[CmdIndex];
	const int32 ElementSize = FastArrayItemCmd.ElementSize;

	UObject* Object = DeltaSerializeInfo.Object;
	UPackageMap* PackageMap = DeltaSerializeInfo.Map;

	FRepObjectDataBuffer ObjectData(Object);
	FScriptArray* ScriptArray = GetTypedProperty<FScriptArray>(ObjectData, FastArrayItemCmd);
	FRepObjectDataBuffer ArrayData(ScriptArray->GetData());

	FFastArraySerializer& ArraySerializer = Params.ArraySerializer;

	for (auto It = ArraySerializer.GuidReferencesMap_StructDelta.CreateIterator(); It; ++It)
	{
		const int32 ElementID = It.Key();
		if (int32 const * const FoundItemIndex = ArraySerializer.ItemMap.Find(ElementID))
		{
			bool bOutSomeObjectsWereMapped = false;
			bool bOutHasMoreUnmapped = false;

			const int32 ItemIndex = *FoundItemIndex;
			const int32 ArrayElementOffset = ItemIndex * ElementSize;
			FRepObjectDataBuffer ElementData(ArrayData + ArrayElementOffset);

			UpdateUnmappedObjects_r(nullptr, &It.Value(), Object, DeltaSerializeInfo.Connection, nullptr, ElementData, ElementSize, Params.DeltaSerializeInfo.bCalledPreNetReceive, bOutSomeObjectsWereMapped, bOutHasMoreUnmapped);

			if (bOutSomeObjectsWereMapped)
			{
				if (Params.ReadChangedElements != nullptr)
				{
					Params.ReadChangedElements->Add(ItemIndex);
				}

				Params.PostReplicatedChange(ElementData, Params);
			}

			DeltaSerializeInfo.bOutHasMoreUnmapped |= bOutHasMoreUnmapped;
			DeltaSerializeInfo.bOutSomeObjectsWereMapped |= bOutSomeObjectsWereMapped;
		}
		else
		{
			It.RemoveCurrent();
		}
	}
}

void FRepLayout::CountBytes(FArchive& Ar) const
{
	GRANULAR_NETWORK_MEMORY_TRACKING_INIT(Ar, "FRepLayout::CountBytes");
	GRANULAR_NETWORK_MEMORY_TRACKING_TRACK("Parents", Parents.CountBytes(Ar));
	GRANULAR_NETWORK_MEMORY_TRACKING_TRACK("Cmds", Cmds.CountBytes(Ar));
	GRANULAR_NETWORK_MEMORY_TRACKING_TRACK("BaseHandleToCmdIndex", BaseHandleToCmdIndex.CountBytes(Ar));
	GRANULAR_NETWORK_MEMORY_TRACKING_TRACK("SharedInfoRPC", SharedInfoRPC.CountBytes(Ar));
	GRANULAR_NETWORK_MEMORY_TRACKING_TRACK("SharedInfoRPCParentsChanged", SharedInfoRPCParentsChanged.CountBytes(Ar));
	GRANULAR_NETWORK_MEMORY_TRACKING_TRACK("LifetimeCustomPropertyState",
		if (LifetimeCustomPropertyState)
		{
			Ar.CountBytes(sizeof(FLifetimeCustomDeltaState), sizeof(FLifetimeCustomDeltaState));
			LifetimeCustomPropertyState->CountBytes(Ar);
		}
	);
	GRANULAR_NETWORK_MEMORY_TRACKING_TRACK("NetSerializeLayouts",
		NetSerializeLayouts.CountBytes(Ar);
		for (auto It = NetSerializeLayouts.CreateConstIterator(); It; ++It)
		{
			It.Value().CountBytes(Ar);
		}
	);
}

const uint16 FRepLayout::GetNumLifetimeCustomDeltaProperties() const
{
	return LifetimeCustomPropertyState.IsValid() ? LifetimeCustomPropertyState->GetNumCustomDeltaProperties() : 0;
}

const uint16 FRepLayout::GetLifetimeCustomDeltaPropertyRepIndex(const uint16 CustomDeltaPropertyIndex) const
{
	checkSlow(LifetimeCustomPropertyState.IsValid());

	return LifetimeCustomPropertyState->GetCustomDeltaProperty(CustomDeltaPropertyIndex).PropertyRepIndex;
}

FProperty* FRepLayout::GetLifetimeCustomDeltaProperty(const uint16 CustomDeltaPropertyIndex) const
{
	checkSlow(LifetimeCustomPropertyState.IsValid());

	const FLifetimeCustomDeltaProperty& CustomDeltaProperty = LifetimeCustomPropertyState->GetCustomDeltaProperty(CustomDeltaPropertyIndex);
	return Parents[CustomDeltaProperty.PropertyRepIndex].Property;
}

const uint16 FRepLayout::GetCustomDeltaIndexFromPropertyRepIndex(const uint16 PropertyRepIndex) const
{
	checkSlow(LifetimeCustomPropertyState.IsValid());

	return LifetimeCustomPropertyState->GetCustomDeltaIndexFromPropertyRepIndex(PropertyRepIndex);
}

const ELifetimeCondition FRepLayout::GetLifetimeCustomDeltaPropertyCondition(const uint16 CustomDeltaPropertyIndex) const
{
	checkSlow(LifetimeCustomPropertyState.IsValid());

	const FLifetimeCustomDeltaProperty& CustomDeltaProperty = LifetimeCustomPropertyState->GetCustomDeltaProperty(CustomDeltaPropertyIndex);
	return Parents[CustomDeltaProperty.PropertyRepIndex].Condition;
}

void FReceivingRepState::CountBytes(FArchive& Ar) const
{
	GRANULAR_NETWORK_MEMORY_TRACKING_INIT(Ar, "FReceivingRepState::CountBytes");

	GRANULAR_NETWORK_MEMORY_TRACKING_TRACK("StaticBuffer", StaticBuffer.CountBytes(Ar));

	GRANULAR_NETWORK_MEMORY_TRACKING_TRACK("GuidReferencesMap",
		GuidReferencesMap.CountBytes(Ar);
		for (const auto& GuidRefPair : GuidReferencesMap)
		{
			GuidRefPair.Value.CountBytes(Ar);
		}
	);

	GRANULAR_NETWORK_MEMORY_TRACKING_TRACK("RepNotifies", RepNotifies.CountBytes(Ar));

	GRANULAR_NETWORK_MEMORY_TRACKING_TRACK("RepNotifyMetaData",
		RepNotifyMetaData.CountBytes(Ar);
		for (const auto& MetaDataPair : RepNotifyMetaData)
		{
			MetaDataPair.Value.CountBytes(Ar);
		}
	);
}

void FSendingRepState::CountBytes(FArchive& Ar) const
{
	// RepChangedPropertyTracker is also stored on the net driver, so it's not tracked here.
	GRANULAR_NETWORK_MEMORY_TRACKING_INIT(Ar, "FSendingRepState::CountBytes");

	GRANULAR_NETWORK_MEMORY_TRACKING_TRACK("ChangeHistory",
		for (const FRepChangedHistory& HistoryItem : ChangeHistory)
		{
			HistoryItem.CountBytes(Ar);
		}
	);

	GRANULAR_NETWORK_MEMORY_TRACKING_TRACK("PreOpenAckHistory",
		PreOpenAckHistory.CountBytes(Ar);
		for (const FRepChangedHistory& HistoryItem : PreOpenAckHistory)
		{
			HistoryItem.CountBytes(Ar);
		}
	);

	GRANULAR_NETWORK_MEMORY_TRACKING_TRACK("LifetimeChangelist", LifetimeChangelist.CountBytes(Ar));
	GRANULAR_NETWORK_MEMORY_TRACKING_TRACK("InactiveChangelist", InactiveChangelist.CountBytes(Ar));
	GRANULAR_NETWORK_MEMORY_TRACKING_TRACK("InactiveParents", InactiveParents.CountBytes(Ar));

	GRANULAR_NETWORK_MEMORY_TRACKING_TRACK("Retirement", Retirement.CountBytes(Ar));

	GRANULAR_NETWORK_MEMORY_TRACKING_TRACK("RecentCustomDeltaState",
		RecentCustomDeltaState.CountBytes(Ar);
		for (const TSharedPtr<INetDeltaBaseState>& LocalRecentCustomDeltaState : RecentCustomDeltaState)
		{
			if (INetDeltaBaseState const * const BaseState = LocalRecentCustomDeltaState.Get())
			{
				BaseState->CountBytes(Ar);
			}
		}
	);

	GRANULAR_NETWORK_MEMORY_TRACKING_TRACK("CDOCustomDeltaState",
		CDOCustomDeltaState.CountBytes(Ar);
		for (const TSharedPtr<INetDeltaBaseState>& LocalRecentCustomDeltaState : CDOCustomDeltaState)
		{
			if (INetDeltaBaseState const* const BaseState = LocalRecentCustomDeltaState.Get())
			{
				BaseState->CountBytes(Ar);
			}
		}
	);

	GRANULAR_NETWORK_MEMORY_TRACKING_TRACK("CheckpointCustomDeltaState",
		CheckpointCustomDeltaState.CountBytes(Ar);
		for (const TSharedPtr<INetDeltaBaseState>& LocalRecentCustomDeltaState : CheckpointCustomDeltaState)
		{
			if (INetDeltaBaseState const* const BaseState = LocalRecentCustomDeltaState.Get())
			{
				BaseState->CountBytes(Ar);
			}
		}
	);
}

void FRepState::CountBytes(FArchive& Ar) const
{
	GRANULAR_NETWORK_MEMORY_TRACKING_INIT(Ar, "FRepState::CountBytes");

	GRANULAR_NETWORK_MEMORY_TRACKING_TRACK("ReceivingRepState",
		if (FReceivingRepState const * const LocalReceivingRepState = ReceivingRepState.Get())
		{
			Ar.CountBytes(sizeof(*LocalReceivingRepState), sizeof(*LocalReceivingRepState));
			LocalReceivingRepState->CountBytes(Ar);
		}
	);	

	GRANULAR_NETWORK_MEMORY_TRACKING_TRACK("SendingRepState",
		if (FSendingRepState const * const LocalSendingRepState = SendingRepState.Get())
		{
			Ar.CountBytes(sizeof(*LocalSendingRepState), sizeof(*LocalSendingRepState));
			LocalSendingRepState->CountBytes(Ar);
		}
	);	
}

FRepStateStaticBuffer::~FRepStateStaticBuffer()
{
	if (Buffer.Num() > 0)
	{
		RepLayout->DestructProperties(*this);
	}

	Buffer = {};
}

const TCHAR* LexToString(ERepLayoutFlags Flag)
{
	switch (Flag)
	{
	case ERepLayoutFlags::IsActor:
		return TEXT("IsActor");
	case ERepLayoutFlags::PartialPushSupport:
		return TEXT("PartialPushSupport");
	case ERepLayoutFlags::FullPushSupport:
		return TEXT("FullPushSupport");
	case ERepLayoutFlags::HasObjectOrNetSerializeProperties:
		return TEXT("HasObjectOrNetSerializeProperties");
	case ERepLayoutFlags::NoReplicatedProperties:
		return TEXT("NoReplicatedProperties");
	case ERepLayoutFlags::FullPushProperties:
		return TEXT("FullPushProperties");
	case ERepLayoutFlags::HasInitialOnlyProperties:
		return TEXT("HasInitialOnlyProperties");
	case ERepLayoutFlags::HasDynamicConditionProperties:
		return TEXT("HasDynamicConditionProperties");
	default:
		check(false);
		return TEXT("Unknown");
	}
}

const TCHAR* LexToString(ERepLayoutCmdType CmdType)
{
	switch (CmdType)
	{
	case ERepLayoutCmdType::DynamicArray:
		return TEXT("DynamicArray");
	case ERepLayoutCmdType::Return:
		return TEXT("Return");
	case ERepLayoutCmdType::Property:
		return TEXT("Property");
	case ERepLayoutCmdType::PropertyBool:
		return TEXT("PropertyBool");
	case ERepLayoutCmdType::PropertyFloat:
		return TEXT("PropertyFloat");
	case ERepLayoutCmdType::PropertyInt:
		return TEXT("PropertyInt");
	case ERepLayoutCmdType::PropertyByte:
		return TEXT("PropertyByte");
	case ERepLayoutCmdType::PropertyName:
		return TEXT("PropertyName");
	case ERepLayoutCmdType::PropertyObject:
		return TEXT("PropertyObject");
	case ERepLayoutCmdType::PropertyUInt32:
		return TEXT("PropertyUInt32");
	case ERepLayoutCmdType::PropertyVector:
		return TEXT("PropertyVector");
	case ERepLayoutCmdType::PropertyRotator:
		return TEXT("PropertyRotator");
	case ERepLayoutCmdType::PropertyPlane:
		return TEXT("PropertyPlane");
	case ERepLayoutCmdType::PropertyVector100:
		return TEXT("PropertyVector100");
	case ERepLayoutCmdType::PropertyNetId:
		return TEXT("PropertyNetId");
	case ERepLayoutCmdType::RepMovement:
		return TEXT("RepMovement");
	case ERepLayoutCmdType::PropertyVectorNormal:
		return TEXT("PropertyVectorNormal");
	case ERepLayoutCmdType::PropertyVector10:
		return TEXT("PropertyVector10");
	case ERepLayoutCmdType::PropertyVectorQ:
		return TEXT("PropertyVectorQ");
	case ERepLayoutCmdType::PropertyString:
		return TEXT("PropertyString");
	case ERepLayoutCmdType::PropertyUInt64:
		return TEXT("PropertyUInt64");
	case ERepLayoutCmdType::PropertyNativeBool:
		return TEXT("PropertyNativeBool");
	case ERepLayoutCmdType::PropertySoftObject:
		return TEXT("PropertySoftObject");
	case ERepLayoutCmdType::PropertyWeakObject:
		return TEXT("PropertyWeakObject");
	case ERepLayoutCmdType::PropertyInterface:
		return TEXT("PropertyInterface");
	case ERepLayoutCmdType::NetSerializeStructWithObjectReferences:
		return TEXT("NetSerializeStructWithObjectReferences");
	default:
		ensureMsgf(false, TEXT("Unhandled layout command type."));
		return TEXT("Unknown");
	}
}

#define REPDATATYPE_SPECIALIZATION(DstType, SrcType) \
template bool FRepLayout::DiffStableProperties(TArray<FProperty*>*, TArray<UObject*>*, TRepDataBuffer<DstType>, TConstRepDataBuffer<SrcType>) const; \
template bool FRepLayout::DiffProperties(TArray<FProperty*>*, TRepDataBuffer<DstType>, TConstRepDataBuffer<SrcType>, const EDiffPropertiesFlags) const;

REPDATATYPE_SPECIALIZATION(ERepDataBufferType::ObjectBuffer, ERepDataBufferType::ObjectBuffer)
REPDATATYPE_SPECIALIZATION(ERepDataBufferType::ObjectBuffer, ERepDataBufferType::ShadowBuffer)
REPDATATYPE_SPECIALIZATION(ERepDataBufferType::ShadowBuffer, ERepDataBufferType::ObjectBuffer)
REPDATATYPE_SPECIALIZATION(ERepDataBufferType::ShadowBuffer, ERepDataBufferType::ShadowBuffer)

#undef REPDATATYPE_SPECIALIZATION

#define REPDATATYPE_SPECIALIZATION(DataType) \
template void FRepLayout::ValidateWithChecksum(TConstRepDataBuffer<DataType> Data, FBitArchive& Ar) const;

REPDATATYPE_SPECIALIZATION(ERepDataBufferType::ShadowBuffer);
REPDATATYPE_SPECIALIZATION(ERepDataBufferType::ObjectBuffer);

#undef REPDATATYPE_SPECIALIZATION
