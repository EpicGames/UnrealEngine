// Copyright Epic Games, Inc. All Rights Reserved.

#include "RepMovementNetSerializer.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RepMovementNetSerializer)

#include "Engine/ReplicatedState.h"
#include "Iris/Core/IrisLog.h"
#include "Iris/Serialization/BitPacking.h"
#include "Iris/Serialization/NetBitStreamReader.h"
#include "Iris/Serialization/NetBitStreamUtil.h"
#include "Iris/Serialization/NetBitStreamWriter.h"
#include "Iris/Serialization/NetSerializerDelegates.h"
#include "Iris/Serialization/PackedVectorNetSerializers.h"
#include "Iris/Serialization/RotatorNetSerializers.h"
#include "Iris/ReplicationState/PropertyNetSerializerInfoRegistry.h"
#include "Net/Core/Trace/NetTrace.h"

namespace UE::Net
{

struct FRepMovementNetSerializer
{
	// Version
	static const uint32 Version = 0;

	// Types
	struct FQuantizedData
	{
		uint64 AngularVelocity[4];
		uint64 LinearVelocity[4];
		uint64 Location[4];
		uint16 Rotation[4];
		uint64 Acceleration[4];
		int32 ServerFrame;
		int32 ServerPhysicsHandle;

		uint16 Flags : 4;
		uint16 VelocityQuantizationLevel : 2;
		uint16 LocationQuantizationLevel : 2;
		uint16 RotationQuantizationLevel : 1;
		uint16 RepAcceleration : 1;
		uint16 Unused : 6;
		uint16 Padding[3];
	};

	typedef FRepMovement SourceType;
	typedef FQuantizedData QuantizedType;
	typedef FRepMovementNetSerializerConfig ConfigType;

	inline static const ConfigType DefaultConfig;

	// 
	static void Serialize(FNetSerializationContext&, const FNetSerializeArgs&);
	static void Deserialize(FNetSerializationContext&, const FNetDeserializeArgs&);

	static void SerializeDelta(FNetSerializationContext&, const FNetSerializeDeltaArgs&);
	static void DeserializeDelta(FNetSerializationContext&, const FNetDeserializeDeltaArgs&);

	static void Quantize(FNetSerializationContext&, const FNetQuantizeArgs&);
	static void Dequantize(FNetSerializationContext&, const FNetDequantizeArgs&);

	static bool IsEqual(FNetSerializationContext&, const FNetIsEqualArgs&);
	static bool Validate(FNetSerializationContext&, const FNetValidateArgs&);

	// Avoid overwriting some members in the target FRepMovement instance.
	static void Apply(FNetSerializationContext&, const FNetApplyArgs&);

private:
	enum EFlags : uint32
	{
		Flag_SimulatedPhysicSleep = 1U,
		Flag_RepPhysics = 2U,
		Flag_ServerFrameIsPresent = 4U,
		Flag_ServerPhysicsHandleIsPresent = 8U,
	};

	// Dynamic precision support for FVector and FRotator
	inline static const FVectorNetQuantizeNetSerializerConfig QuantizeSerializerConfig;
	inline static const FVectorNetQuantize10NetSerializerConfig Quantize10SerializerConfig;
	inline static const FVectorNetQuantize100NetSerializerConfig Quantize100SerializerConfig;

	inline static const FRotatorAsByteNetSerializerConfig RotatorAsByteSerializerConfig;
	inline static const FRotatorAsShortNetSerializerConfig RotatorAsShortSerializerConfig;

	inline static const FNetSerializerConfig* VectorNetQuantizeNetSerializerConfigs[4] = {&QuantizeSerializerConfig, &Quantize10SerializerConfig, &Quantize100SerializerConfig, nullptr};
	inline static const FNetSerializer* VectorNetQuantizeNetSerializers[4] = {};

	inline static const FNetSerializerConfig* RotatorNetSerializerConfigs[2] = {&RotatorAsByteSerializerConfig, &RotatorAsShortSerializerConfig};
	inline static const FNetSerializer* RotatorNetSerializers[2] = {};

private:
	static constexpr SIZE_T DeltaBitCountTableEntryCount = 3;
	// Bit counts aiming to have small value changes use few bits.
	static constexpr uint8 DeltaBitCountTable[] = {0, 4, 14};

	class FNetSerializerRegistryDelegates final : private UE::Net::FNetSerializerRegistryDelegates
	{
	public:
		virtual ~FNetSerializerRegistryDelegates();

	private:
		virtual void OnPreFreezeNetSerializerRegistry() override;

		bool QuantizedTypeMeetRequirements() const;
		bool IsRepMovementLayoutAsExpected() const;
		void InitNetSerializer();

		inline static const FName RepMovementNetSerializer = FName("RepMovement");
		UE_NET_IMPLEMENT_NAMED_STRUCT_NETSERIALIZER_INFO(RepMovementNetSerializer, FRepMovementNetSerializer);
	};

	inline static FRepMovementNetSerializer::FNetSerializerRegistryDelegates NetSerializerRegistryDelegates;
};

UE_NET_IMPLEMENT_SERIALIZER(FRepMovementNetSerializer);

// FRepMovementNetSerializer implementation
void FRepMovementNetSerializer::Serialize(FNetSerializationContext& Context, const FNetSerializeArgs& Args)
{
	const QuantizedType& Value = *reinterpret_cast<const QuantizedType*>(Args.Source);

	FNetBitStreamWriter* Writer = Context.GetBitStreamWriter();

	constexpr uint32 QuantizationLevelsBitOffset = 4;
	const uint32 FlagsAndQuantizationLevels = (((Value.RotationQuantizationLevel << 4U) | (Value.LocationQuantizationLevel << 2U) | Value.VelocityQuantizationLevel) << QuantizationLevelsBitOffset) | Value.Flags;
	if (Writer->WriteBool(FlagsAndQuantizationLevels != 0))
	{
		UE_NET_TRACE_SCOPE(FlagsAndQuantizationLevels, *Writer, Context.GetTraceCollector(), ENetTraceVerbosity::Verbose);
		constexpr uint32 QuantizationLevelsBitCount = 5U;
		Writer->WriteBits(FlagsAndQuantizationLevels, QuantizationLevelsBitCount + QuantizationLevelsBitOffset);
	}

	// Angular velocity
	if (Value.Flags & Flag_RepPhysics)
	{
		UE_NET_TRACE_SCOPE(AngularVelocity, *Writer, Context.GetTraceCollector(), ENetTraceVerbosity::Verbose);
		const FNetSerializer* Serializer = VectorNetQuantizeNetSerializers[Value.VelocityQuantizationLevel];
		const FNetSerializerConfig* SerializerConfig = VectorNetQuantizeNetSerializerConfigs[Value.VelocityQuantizationLevel];

		FNetSerializeArgs MemberArgs = Args;
		MemberArgs.NetSerializerConfig = NetSerializerConfigParam(SerializerConfig);
		MemberArgs.Source = NetSerializerValuePointer(&Value.AngularVelocity[0]);
		Serializer->Serialize(Context, MemberArgs);
	}

	// Linear velocity
	{
		UE_NET_TRACE_SCOPE(LinearVelocity, *Writer, Context.GetTraceCollector(), ENetTraceVerbosity::Verbose);
		const FNetSerializer* Serializer = VectorNetQuantizeNetSerializers[Value.VelocityQuantizationLevel];
		const FNetSerializerConfig* SerializerConfig = VectorNetQuantizeNetSerializerConfigs[Value.VelocityQuantizationLevel];

		FNetSerializeArgs MemberArgs = Args;
		MemberArgs.NetSerializerConfig = NetSerializerConfigParam(SerializerConfig);
		MemberArgs.Source = NetSerializerValuePointer(&Value.LinearVelocity[0]);
		Serializer->Serialize(Context, MemberArgs);
	}

	// Location
	{
		UE_NET_TRACE_SCOPE(Location, *Writer, Context.GetTraceCollector(), ENetTraceVerbosity::Verbose);
		const FNetSerializer* Serializer = VectorNetQuantizeNetSerializers[Value.LocationQuantizationLevel];
		const FNetSerializerConfig* SerializerConfig = VectorNetQuantizeNetSerializerConfigs[Value.LocationQuantizationLevel];

		FNetSerializeArgs MemberArgs = Args;
		MemberArgs.NetSerializerConfig = NetSerializerConfigParam(SerializerConfig);
		MemberArgs.Source = NetSerializerValuePointer(&Value.Location[0]);
		Serializer->Serialize(Context, MemberArgs);
	}

	// Rotation
	{
		UE_NET_TRACE_SCOPE(Rotation, *Writer, Context.GetTraceCollector(), ENetTraceVerbosity::Verbose);
		const FNetSerializer* Serializer = RotatorNetSerializers[Value.RotationQuantizationLevel];
		const FNetSerializerConfig* SerializerConfig = RotatorNetSerializerConfigs[Value.RotationQuantizationLevel];

		FNetSerializeArgs MemberArgs = Args;
		MemberArgs.NetSerializerConfig = NetSerializerConfigParam(SerializerConfig);
		MemberArgs.Source = NetSerializerValuePointer(&Value.Rotation[0]);
		Serializer->Serialize(Context, MemberArgs);
	}

	// ServerFrame
	if (Value.Flags & Flag_ServerFrameIsPresent)
	{
		WritePackedInt32(Writer, Value.ServerFrame);
	}

	// ServerPhsyicsHandle
	if (Value.Flags & Flag_ServerPhysicsHandleIsPresent)
	{
		WritePackedInt32(Writer, Value.ServerPhysicsHandle);
	}

	// Optional acceleration
	{
		// FUTURE: Consider checking EngineNetVer >= FEngineNetworkCustomVersion::RepMoveOptionalAcceleration for backwards compatibility
		if (Writer->WriteBool(Value.RepAcceleration))
		{
			UE_NET_TRACE_SCOPE(Acceleration, *Writer, Context.GetTraceCollector(), ENetTraceVerbosity::Verbose);
			const FNetSerializer* Serializer = VectorNetQuantizeNetSerializers[Value.VelocityQuantizationLevel];	// Note that we're using the same quantization as Velocity, since the numbers are typically on the same order
			const FNetSerializerConfig* SerializerConfig = VectorNetQuantizeNetSerializerConfigs[Value.VelocityQuantizationLevel];

			FNetSerializeArgs MemberArgs = Args;
			MemberArgs.NetSerializerConfig = NetSerializerConfigParam(SerializerConfig);
			MemberArgs.Source = NetSerializerValuePointer(&Value.Acceleration[0]);
			Serializer->Serialize(Context, MemberArgs);
		}
	}

}

void FRepMovementNetSerializer::Deserialize(FNetSerializationContext& Context, const FNetDeserializeArgs& Args)
{
	QuantizedType TempValue = {};

	FNetBitStreamReader* Reader = Context.GetBitStreamReader();
	if (Reader->ReadBool())
	{
		UE_NET_TRACE_SCOPE(FlagsAndQuantizationLevels, *Reader, Context.GetTraceCollector(), ENetTraceVerbosity::Verbose);
		const uint32 FlagsAndQuantizationLevels = Reader->ReadBits(9U);
		constexpr uint32 QuantizationLevelsBitOffset = 4U;
		TempValue.Flags = (FlagsAndQuantizationLevels & ((1U << QuantizationLevelsBitOffset) - 1U));
		TempValue.VelocityQuantizationLevel = (FlagsAndQuantizationLevels >> (QuantizationLevelsBitOffset + 0U)) & 3U;
		TempValue.LocationQuantizationLevel = (FlagsAndQuantizationLevels >> (QuantizationLevelsBitOffset + 2U)) & 3U;
		TempValue.RotationQuantizationLevel = (FlagsAndQuantizationLevels >> (QuantizationLevelsBitOffset + 4U)) & 1U;

		if (VectorNetQuantizeNetSerializers[TempValue.VelocityQuantizationLevel] == nullptr || RotatorNetSerializers[TempValue.RotationQuantizationLevel] == nullptr)
		{
			Context.SetError(GNetError_InvalidValue);
			return;
		}
	}

	// Angular velocity
	if (TempValue.Flags & Flag_RepPhysics)
	{
		UE_NET_TRACE_SCOPE(AngularVelocity, *Reader, Context.GetTraceCollector(), ENetTraceVerbosity::Verbose);
		const FNetSerializer* Serializer = VectorNetQuantizeNetSerializers[TempValue.VelocityQuantizationLevel];
		const FNetSerializerConfig* SerializerConfig = VectorNetQuantizeNetSerializerConfigs[TempValue.VelocityQuantizationLevel];

		FNetDeserializeArgs MemberArgs = Args;
		MemberArgs.NetSerializerConfig = NetSerializerConfigParam(SerializerConfig);
		MemberArgs.Target = NetSerializerValuePointer(&TempValue.AngularVelocity[0]);
		Serializer->Deserialize(Context, MemberArgs);
	}

	// Linear velocity
	{
		UE_NET_TRACE_SCOPE(LinearVelocity, *Reader, Context.GetTraceCollector(), ENetTraceVerbosity::Verbose);
		const FNetSerializer* Serializer = VectorNetQuantizeNetSerializers[TempValue.VelocityQuantizationLevel];
		const FNetSerializerConfig* SerializerConfig = VectorNetQuantizeNetSerializerConfigs[TempValue.VelocityQuantizationLevel];

		FNetDeserializeArgs MemberArgs = Args;
		MemberArgs.NetSerializerConfig = NetSerializerConfigParam(SerializerConfig);
		MemberArgs.Target = NetSerializerValuePointer(&TempValue.LinearVelocity[0]);
		Serializer->Deserialize(Context, MemberArgs);
	}

	// Location
	{
		UE_NET_TRACE_SCOPE(Location, *Reader, Context.GetTraceCollector(), ENetTraceVerbosity::Verbose);
		const FNetSerializer* Serializer = VectorNetQuantizeNetSerializers[TempValue.LocationQuantizationLevel];
		const FNetSerializerConfig* SerializerConfig = VectorNetQuantizeNetSerializerConfigs[TempValue.LocationQuantizationLevel];

		FNetDeserializeArgs MemberArgs = Args;
		MemberArgs.NetSerializerConfig = NetSerializerConfigParam(SerializerConfig);
		MemberArgs.Target = NetSerializerValuePointer(&TempValue.Location[0]);
		Serializer->Deserialize(Context, MemberArgs);
	}

	// Rotation
	{
		UE_NET_TRACE_SCOPE(Rotation, *Reader, Context.GetTraceCollector(), ENetTraceVerbosity::Verbose);
		const FNetSerializer* Serializer = RotatorNetSerializers[TempValue.RotationQuantizationLevel];
		const FNetSerializerConfig* SerializerConfig = RotatorNetSerializerConfigs[TempValue.RotationQuantizationLevel];

		FNetDeserializeArgs MemberArgs = Args;
		MemberArgs.NetSerializerConfig = NetSerializerConfigParam(SerializerConfig);
		MemberArgs.Target = NetSerializerValuePointer(&TempValue.Rotation[0]);
		Serializer->Deserialize(Context, MemberArgs);
	}

	// ServerFrame
	if (TempValue.Flags & Flag_ServerFrameIsPresent)
	{
		TempValue.ServerFrame = ReadPackedInt32(Reader);
	}
	else
	{
		TempValue.ServerFrame = 0;
	}

	// ServerPhysicsHandle
	if (TempValue.Flags & Flag_ServerPhysicsHandleIsPresent)
	{
		TempValue.ServerPhysicsHandle = ReadPackedInt32(Reader);
	}
	else
	{
		TempValue.ServerPhysicsHandle = INDEX_NONE;
	}

	// Optional acceleration
	{
		// FUTURE: Consider checking EngineNetVer >= FEngineNetworkCustomVersion::RepMoveOptionalAcceleration for backwards compatibility
		TempValue.RepAcceleration = Reader->ReadBool();

		if (TempValue.RepAcceleration)
		{
			UE_NET_TRACE_SCOPE(Acceleration, *Reader, Context.GetTraceCollector(), ENetTraceVerbosity::Verbose);
			const FNetSerializer* Serializer = VectorNetQuantizeNetSerializers[TempValue.VelocityQuantizationLevel];	// Note that we're using the same quantization as Velocity, since the numbers are typically on the same order
			const FNetSerializerConfig* SerializerConfig = VectorNetQuantizeNetSerializerConfigs[TempValue.VelocityQuantizationLevel];

			FNetDeserializeArgs MemberArgs = Args;
			MemberArgs.NetSerializerConfig = NetSerializerConfigParam(SerializerConfig);
			MemberArgs.Target = NetSerializerValuePointer(&TempValue.Acceleration[0]);
			Serializer->Deserialize(Context, MemberArgs);
		}
	}


	QuantizedType& Target = *reinterpret_cast<QuantizedType*>(Args.Target);
	Target = TempValue;
}

void FRepMovementNetSerializer::SerializeDelta(FNetSerializationContext& Context, const FNetSerializeDeltaArgs& Args)
{
	const QuantizedType& Value = *reinterpret_cast<const QuantizedType*>(Args.Source);
	const QuantizedType& PrevValue = *reinterpret_cast<const QuantizedType*>(Args.Prev);

	FNetBitStreamWriter* Writer = Context.GetBitStreamWriter();

	// If the quantization levels change we cannot perform meaningful delta compression. This is highly unexpected so we fallback to regular serialization.
	const bool bCanDeltaCompress = (Value.VelocityQuantizationLevel == PrevValue.VelocityQuantizationLevel) & (Value.LocationQuantizationLevel == PrevValue.LocationQuantizationLevel) & (Value.RotationQuantizationLevel == PrevValue.RotationQuantizationLevel);
	if (!Writer->WriteBool(bCanDeltaCompress))
	{
		Serialize(Context, Args);
		return;
	}

	// We know the quantization levels are equal so we only need to care about the flags.
	uint32 Flags = Value.Flags & (Flag_SimulatedPhysicSleep | Flag_RepPhysics);
	Flags |= (Value.ServerFrame != PrevValue.ServerFrame ? Flag_ServerFrameIsPresent : 0U);
	Flags |= (Value.ServerPhysicsHandle != PrevValue.ServerPhysicsHandle ? Flag_ServerPhysicsHandleIsPresent : 0U);

	Writer->WriteBits(Flags, 4U);

	// Angular velocity
	if (Flags & Flag_RepPhysics)
	{
		UE_NET_TRACE_SCOPE(AngularVelocity, *Writer, Context.GetTraceCollector(), ENetTraceVerbosity::Verbose);
		const FNetSerializer* Serializer = VectorNetQuantizeNetSerializers[Value.VelocityQuantizationLevel];
		const FNetSerializerConfig* SerializerConfig = VectorNetQuantizeNetSerializerConfigs[Value.VelocityQuantizationLevel];

		FNetSerializeDeltaArgs MemberArgs = Args;
		MemberArgs.NetSerializerConfig = NetSerializerConfigParam(SerializerConfig);
		MemberArgs.Source = NetSerializerValuePointer(&Value.AngularVelocity[0]);
		MemberArgs.Prev = NetSerializerValuePointer(&PrevValue.AngularVelocity[0]);

		// It seems only meaningful to delta serialize if both the current value and the previous value have valid angular velocities.
		if (PrevValue.Flags & Flag_RepPhysics)
		{
			Serializer->SerializeDelta(Context, MemberArgs);
		}
		else
		{
			Serializer->Serialize(Context, MemberArgs);
		}
	}

	// Linear velocity
	{
		UE_NET_TRACE_SCOPE(LinearVelocity, *Writer, Context.GetTraceCollector(), ENetTraceVerbosity::Verbose);
		const FNetSerializer* Serializer = VectorNetQuantizeNetSerializers[Value.VelocityQuantizationLevel];
		const FNetSerializerConfig* SerializerConfig = VectorNetQuantizeNetSerializerConfigs[Value.VelocityQuantizationLevel];

		FNetSerializeDeltaArgs MemberArgs = Args;
		MemberArgs.NetSerializerConfig = NetSerializerConfigParam(SerializerConfig);
		MemberArgs.Source = NetSerializerValuePointer(&Value.LinearVelocity[0]);
		MemberArgs.Prev = NetSerializerValuePointer(&PrevValue.LinearVelocity[0]);
		Serializer->SerializeDelta(Context, MemberArgs);
	}

	// Location
	{
		UE_NET_TRACE_SCOPE(Location, *Writer, Context.GetTraceCollector(), ENetTraceVerbosity::Verbose);
		const FNetSerializer* Serializer = VectorNetQuantizeNetSerializers[Value.LocationQuantizationLevel];
		const FNetSerializerConfig* SerializerConfig = VectorNetQuantizeNetSerializerConfigs[Value.LocationQuantizationLevel];

		FNetSerializeDeltaArgs MemberArgs = Args;
		MemberArgs.NetSerializerConfig = NetSerializerConfigParam(SerializerConfig);
		MemberArgs.Source = NetSerializerValuePointer(&Value.Location[0]);
		MemberArgs.Prev = NetSerializerValuePointer(&PrevValue.Location[0]);
		Serializer->SerializeDelta(Context, MemberArgs);
	}

	// Rotation
	{
		UE_NET_TRACE_SCOPE(Rotation, *Writer, Context.GetTraceCollector(), ENetTraceVerbosity::Verbose);
		const FNetSerializer* Serializer = RotatorNetSerializers[Value.RotationQuantizationLevel];
		const FNetSerializerConfig* SerializerConfig = RotatorNetSerializerConfigs[Value.RotationQuantizationLevel];

		FNetSerializeDeltaArgs MemberArgs = Args;
		MemberArgs.NetSerializerConfig = NetSerializerConfigParam(SerializerConfig);
		MemberArgs.Source = NetSerializerValuePointer(&Value.Rotation[0]);
		MemberArgs.Prev = NetSerializerValuePointer(&PrevValue.Rotation[0]);
		Serializer->SerializeDelta(Context, MemberArgs);
	}

	if (Flags & Flag_ServerFrameIsPresent)
	{
		SerializeIntDelta(*Writer, Value.ServerFrame, PrevValue.ServerFrame, DeltaBitCountTable, DeltaBitCountTableEntryCount, 32U);
	}

	if (Flags & Flag_ServerPhysicsHandleIsPresent)
	{
		SerializeIntDelta(*Writer, Value.ServerPhysicsHandle, PrevValue.ServerPhysicsHandle, DeltaBitCountTable, DeltaBitCountTableEntryCount, 32U);
	}

	// Optional acceleration
	{
		// FUTURE: Consider checking EngineNetVer >= FEngineNetworkCustomVersion::RepMoveOptionalAcceleration for backwards compatibility
		Writer->WriteBool(Value.RepAcceleration);

		if (Value.RepAcceleration)
		{
			UE_NET_TRACE_SCOPE(Acceleration, *Writer, Context.GetTraceCollector(), ENetTraceVerbosity::Verbose);
			const FNetSerializer* Serializer = VectorNetQuantizeNetSerializers[Value.VelocityQuantizationLevel];
			const FNetSerializerConfig* SerializerConfig = VectorNetQuantizeNetSerializerConfigs[Value.VelocityQuantizationLevel];

			FNetSerializeDeltaArgs MemberArgs = Args;
			MemberArgs.NetSerializerConfig = NetSerializerConfigParam(SerializerConfig);
			MemberArgs.Source = NetSerializerValuePointer(&Value.Acceleration[0]);
			MemberArgs.Prev = NetSerializerValuePointer(&PrevValue.Acceleration[0]);

			if (PrevValue.RepAcceleration)
			{
				Serializer->SerializeDelta(Context, MemberArgs);
			}
			else // since previously we have no acceleration data, fully serialize it
			{
				Serializer->Serialize(Context, MemberArgs);
			}
		}
	}
}

void FRepMovementNetSerializer::DeserializeDelta(FNetSerializationContext& Context, const FNetDeserializeDeltaArgs& Args)
{
	FNetBitStreamReader* Reader = Context.GetBitStreamReader();

	// If no delta compression was performed just forward to Deserialize.
	if (!Reader->ReadBool())
	{
		Deserialize(Context, Args);
		return;
	}

	const QuantizedType& PrevValue = *reinterpret_cast<const QuantizedType*>(Args.Prev);

	QuantizedType TempValue = {};
	TempValue.VelocityQuantizationLevel = PrevValue.VelocityQuantizationLevel;
	TempValue.LocationQuantizationLevel = PrevValue.LocationQuantizationLevel;
	TempValue.RotationQuantizationLevel = PrevValue.RotationQuantizationLevel;
	const uint32 Flags = Reader->ReadBits(4U);

	// Angular velocity
	if (Flags & Flag_RepPhysics)
	{
		UE_NET_TRACE_SCOPE(AngularVelocity, *Reader, Context.GetTraceCollector(), ENetTraceVerbosity::Verbose);
		const FNetSerializer* Serializer = VectorNetQuantizeNetSerializers[TempValue.VelocityQuantizationLevel];
		const FNetSerializerConfig* SerializerConfig = VectorNetQuantizeNetSerializerConfigs[TempValue.VelocityQuantizationLevel];

		FNetDeserializeDeltaArgs MemberArgs = Args;
		MemberArgs.NetSerializerConfig = NetSerializerConfigParam(SerializerConfig);
		MemberArgs.Target = NetSerializerValuePointer(&TempValue.AngularVelocity[0]);
		MemberArgs.Prev = NetSerializerValuePointer(&PrevValue.AngularVelocity[0]);

		// It seems only meaningful to delta serialize if both the current value and the previous value have valid angular velocities.
		if (PrevValue.Flags & Flag_RepPhysics)
		{
			Serializer->DeserializeDelta(Context, MemberArgs);
		}
		else
		{
			Serializer->Deserialize(Context, MemberArgs);
		}
	}

	// Linear velocity
	{
		UE_NET_TRACE_SCOPE(LinearVelocity, *Reader, Context.GetTraceCollector(), ENetTraceVerbosity::Verbose);
		const FNetSerializer* Serializer = VectorNetQuantizeNetSerializers[TempValue.VelocityQuantizationLevel];
		const FNetSerializerConfig* SerializerConfig = VectorNetQuantizeNetSerializerConfigs[TempValue.VelocityQuantizationLevel];

		FNetDeserializeDeltaArgs MemberArgs = Args;
		MemberArgs.NetSerializerConfig = NetSerializerConfigParam(SerializerConfig);
		MemberArgs.Target = NetSerializerValuePointer(&TempValue.LinearVelocity[0]);
		MemberArgs.Prev = NetSerializerValuePointer(&PrevValue.LinearVelocity[0]);
		Serializer->DeserializeDelta(Context, MemberArgs);
	}

	// Location
	{
		UE_NET_TRACE_SCOPE(Location, *Reader, Context.GetTraceCollector(), ENetTraceVerbosity::Verbose);
		const FNetSerializer* Serializer = VectorNetQuantizeNetSerializers[TempValue.LocationQuantizationLevel];
		const FNetSerializerConfig* SerializerConfig = VectorNetQuantizeNetSerializerConfigs[TempValue.LocationQuantizationLevel];

		FNetDeserializeDeltaArgs MemberArgs = Args;
		MemberArgs.NetSerializerConfig = NetSerializerConfigParam(SerializerConfig);
		MemberArgs.Target = NetSerializerValuePointer(&TempValue.Location[0]);
		MemberArgs.Prev = NetSerializerValuePointer(&PrevValue.Location[0]);
		Serializer->DeserializeDelta(Context, MemberArgs);
	}

	// Rotation
	{
		UE_NET_TRACE_SCOPE(Rotation, *Reader, Context.GetTraceCollector(), ENetTraceVerbosity::Verbose);
		const FNetSerializer* Serializer = RotatorNetSerializers[TempValue.RotationQuantizationLevel];
		const FNetSerializerConfig* SerializerConfig = RotatorNetSerializerConfigs[TempValue.RotationQuantizationLevel];

		FNetDeserializeDeltaArgs MemberArgs = Args;
		MemberArgs.NetSerializerConfig = NetSerializerConfigParam(SerializerConfig);
		MemberArgs.Target = NetSerializerValuePointer(&TempValue.Rotation[0]);
		MemberArgs.Prev = NetSerializerValuePointer(&PrevValue.Rotation[0]);
		Serializer->DeserializeDelta(Context, MemberArgs);
	}

	if (Flags & Flag_ServerFrameIsPresent)
	{
		DeserializeIntDelta(*Reader, TempValue.ServerFrame, PrevValue.ServerFrame, DeltaBitCountTable, DeltaBitCountTableEntryCount, 32U);
	}
	else
	{
		TempValue.ServerFrame = PrevValue.ServerFrame;
	}

	if (Flags & Flag_ServerPhysicsHandleIsPresent)
	{
		DeserializeIntDelta(*Reader, TempValue.ServerPhysicsHandle, PrevValue.ServerPhysicsHandle, DeltaBitCountTable, DeltaBitCountTableEntryCount, 32U);
	}
	else
	{
		TempValue.ServerPhysicsHandle = PrevValue.ServerPhysicsHandle;
	}

	// Fixup flags
	uint32 FixedUpFlags = Flags & (Flag_RepPhysics | Flag_SimulatedPhysicSleep);
	FixedUpFlags |= (TempValue.ServerFrame > 0 ? Flag_ServerFrameIsPresent : 0U);
	FixedUpFlags |= (TempValue.ServerPhysicsHandle != INDEX_NONE ? Flag_ServerPhysicsHandleIsPresent : 0U);
	TempValue.Flags = FixedUpFlags;

	// Optional acceleration
	{
		// FUTURE: Consider checking EngineNetVer >= FEngineNetworkCustomVersion::RepMoveOptionalAcceleration for backwards compatibility
		UE_NET_TRACE_SCOPE(Acceleration, *Reader, Context.GetTraceCollector(), ENetTraceVerbosity::Verbose);
		const FNetSerializer* Serializer = VectorNetQuantizeNetSerializers[TempValue.VelocityQuantizationLevel];
		const FNetSerializerConfig* SerializerConfig = VectorNetQuantizeNetSerializerConfigs[TempValue.VelocityQuantizationLevel];

		FNetDeserializeDeltaArgs MemberArgs = Args;
		MemberArgs.NetSerializerConfig = NetSerializerConfigParam(SerializerConfig);
		MemberArgs.Target = NetSerializerValuePointer(&TempValue.Acceleration[0]);
		MemberArgs.Prev = NetSerializerValuePointer(&PrevValue.Acceleration[0]);

		TempValue.RepAcceleration = Reader->ReadBool();

		if (TempValue.RepAcceleration)
		{
			if (PrevValue.RepAcceleration)
			{
				Serializer->DeserializeDelta(Context, MemberArgs);
			}
			else
			{
				// No prior value, so we deserialize the whole value
				Serializer->Deserialize(Context, MemberArgs);
			}
		}
	}


	QuantizedType& Target = *reinterpret_cast<QuantizedType*>(Args.Target);
	Target = TempValue;
}

void FRepMovementNetSerializer::Quantize(FNetSerializationContext& Context, const FNetQuantizeArgs& Args)
{
	const SourceType& Source = *reinterpret_cast<const SourceType*>(Args.Source);

	QuantizedType TempValue = {};
	TempValue.Flags = (Source.bSimulatedPhysicSleep ? Flag_SimulatedPhysicSleep : 0U) | (Source.bRepPhysics ? Flag_RepPhysics : 0);
	TempValue.Flags |= (Source.ServerFrame > 0 ? Flag_ServerFrameIsPresent : 0U);
	TempValue.Flags |= (Source.ServerPhysicsHandle != INDEX_NONE ? Flag_ServerPhysicsHandleIsPresent : 0U);
	TempValue.LocationQuantizationLevel = uint8(Source.LocationQuantizationLevel);
	TempValue.VelocityQuantizationLevel = uint8(Source.VelocityQuantizationLevel);
	TempValue.RotationQuantizationLevel = uint8(Source.RotationQuantizationLevel);
	TempValue.ServerFrame = Source.ServerFrame;
	TempValue.ServerPhysicsHandle = Source.ServerPhysicsHandle;
	TempValue.RepAcceleration = Source.bRepAcceleration;
	
	// Angular velocity. Do note that the quantized value is cleared if RepPhysics is false. Delta compression need to accommodate to this.
	if (TempValue.Flags & Flag_RepPhysics)
	{
		const FNetSerializer* Serializer = VectorNetQuantizeNetSerializers[TempValue.VelocityQuantizationLevel];
		const FNetSerializerConfig* SerializerConfig = VectorNetQuantizeNetSerializerConfigs[TempValue.VelocityQuantizationLevel];

		FNetQuantizeArgs MemberArgs = Args;
		MemberArgs.NetSerializerConfig = NetSerializerConfigParam(SerializerConfig);
		MemberArgs.Source = NetSerializerValuePointer(&Source.AngularVelocity);
		MemberArgs.Target = NetSerializerValuePointer(&TempValue.AngularVelocity[0]);
		Serializer->Quantize(Context, MemberArgs);
	}

	// Linear velocity
	{
		const FNetSerializer* Serializer = VectorNetQuantizeNetSerializers[TempValue.VelocityQuantizationLevel];
		const FNetSerializerConfig* SerializerConfig = VectorNetQuantizeNetSerializerConfigs[TempValue.VelocityQuantizationLevel];

		FNetQuantizeArgs MemberArgs = Args;
		MemberArgs.NetSerializerConfig = NetSerializerConfigParam(SerializerConfig);
		MemberArgs.Source = NetSerializerValuePointer(&Source.LinearVelocity);
		MemberArgs.Target = NetSerializerValuePointer(&TempValue.LinearVelocity[0]);
		Serializer->Quantize(Context, MemberArgs);
	}

	// Location
	{
		const FNetSerializer* Serializer = VectorNetQuantizeNetSerializers[TempValue.LocationQuantizationLevel];
		const FNetSerializerConfig* SerializerConfig = VectorNetQuantizeNetSerializerConfigs[TempValue.LocationQuantizationLevel];

		FNetQuantizeArgs MemberArgs = Args;
		MemberArgs.NetSerializerConfig = NetSerializerConfigParam(SerializerConfig);
		MemberArgs.Source = NetSerializerValuePointer(&Source.Location);
		MemberArgs.Target = NetSerializerValuePointer(&TempValue.Location[0]);
		Serializer->Quantize(Context, MemberArgs);
	}

	// Rotation
	{
		const FNetSerializer* Serializer = RotatorNetSerializers[TempValue.RotationQuantizationLevel];
		const FNetSerializerConfig* SerializerConfig = RotatorNetSerializerConfigs[TempValue.RotationQuantizationLevel];

		FNetQuantizeArgs MemberArgs = Args;
		MemberArgs.NetSerializerConfig = NetSerializerConfigParam(SerializerConfig);
		MemberArgs.Source = NetSerializerValuePointer(&Source.Rotation);
		MemberArgs.Target = NetSerializerValuePointer(&TempValue.Rotation[0]);
		Serializer->Quantize(Context, MemberArgs);
	}

	// Optional acceleration
	if (TempValue.RepAcceleration)
	{
		// Note that we're using the same quantization settings as linear velocity, since the values are typically on the same order
		const FNetSerializer* Serializer = VectorNetQuantizeNetSerializers[TempValue.VelocityQuantizationLevel];
		const FNetSerializerConfig* SerializerConfig = VectorNetQuantizeNetSerializerConfigs[TempValue.VelocityQuantizationLevel];

		FNetQuantizeArgs MemberArgs = Args;
		MemberArgs.NetSerializerConfig = NetSerializerConfigParam(SerializerConfig);
		MemberArgs.Source = NetSerializerValuePointer(&Source.Acceleration);
		MemberArgs.Target = NetSerializerValuePointer(&TempValue.Acceleration[0]);
		Serializer->Quantize(Context, MemberArgs);
	}

	QuantizedType& Target = *reinterpret_cast<QuantizedType*>(Args.Target);
	Target = TempValue;
}

void FRepMovementNetSerializer::Dequantize(FNetSerializationContext& Context, const FNetDequantizeArgs& Args)
{
	const QuantizedType& Source = *reinterpret_cast<const QuantizedType*>(Args.Source);
	SourceType& Target = *reinterpret_cast<SourceType*>(Args.Target);

	Target.bSimulatedPhysicSleep = (Source.Flags & Flag_SimulatedPhysicSleep) ? 1 : 0;
	Target.bRepPhysics = (Source.Flags & Flag_RepPhysics) ? 1 : 0;
	Target.ServerFrame = (Source.Flags & Flag_ServerFrameIsPresent ? Source.ServerFrame : 0);
	Target.ServerPhysicsHandle = (Source.Flags & Flag_ServerPhysicsHandleIsPresent ? Source.ServerPhysicsHandle : INDEX_NONE);
	Target.LocationQuantizationLevel = EVectorQuantization(Source.LocationQuantizationLevel);
	Target.VelocityQuantizationLevel = EVectorQuantization(Source.VelocityQuantizationLevel);
	Target.RotationQuantizationLevel = ERotatorQuantization(Source.RotationQuantizationLevel);
	Target.bRepAcceleration = Source.RepAcceleration;

	// There's no need to dequantize angular velocity unless we're replicating it.
	if (Source.Flags & Flag_RepPhysics)
	{
		const FNetSerializer* Serializer = VectorNetQuantizeNetSerializers[Source.VelocityQuantizationLevel];
		const FNetSerializerConfig* SerializerConfig = VectorNetQuantizeNetSerializerConfigs[Source.VelocityQuantizationLevel];

		FNetDequantizeArgs MemberArgs = Args;
		MemberArgs.NetSerializerConfig = NetSerializerConfigParam(SerializerConfig);
		MemberArgs.Source = NetSerializerValuePointer(&Source.AngularVelocity[0]);
		MemberArgs.Target = NetSerializerValuePointer(&Target.AngularVelocity);
		Serializer->Dequantize(Context, MemberArgs);
	}

	// Linear velocity
	{
		const FNetSerializer* Serializer = VectorNetQuantizeNetSerializers[Source.VelocityQuantizationLevel];
		const FNetSerializerConfig* SerializerConfig = VectorNetQuantizeNetSerializerConfigs[Source.VelocityQuantizationLevel];

		FNetDequantizeArgs MemberArgs = Args;
		MemberArgs.NetSerializerConfig = NetSerializerConfigParam(SerializerConfig);
		MemberArgs.Source = NetSerializerValuePointer(&Source.LinearVelocity[0]);
		MemberArgs.Target = NetSerializerValuePointer(&Target.LinearVelocity);
		Serializer->Dequantize(Context, MemberArgs);
	}

	// Location
	{
		const FNetSerializer* Serializer = VectorNetQuantizeNetSerializers[Source.LocationQuantizationLevel];
		const FNetSerializerConfig* SerializerConfig = VectorNetQuantizeNetSerializerConfigs[Source.LocationQuantizationLevel];

		FNetDequantizeArgs MemberArgs = Args;
		MemberArgs.NetSerializerConfig = NetSerializerConfigParam(SerializerConfig);
		MemberArgs.Source = NetSerializerValuePointer(&Source.Location[0]);
		MemberArgs.Target = NetSerializerValuePointer(&Target.Location);
		Serializer->Dequantize(Context, MemberArgs);
	}

	// Rotation
	{
		const FNetSerializer* Serializer = RotatorNetSerializers[Source.RotationQuantizationLevel];
		const FNetSerializerConfig* SerializerConfig = RotatorNetSerializerConfigs[Source.RotationQuantizationLevel];

		FNetDequantizeArgs MemberArgs = Args;
		MemberArgs.NetSerializerConfig = NetSerializerConfigParam(SerializerConfig);
		MemberArgs.Source = NetSerializerValuePointer(&Source.Rotation[0]);
		MemberArgs.Target = NetSerializerValuePointer(&Target.Rotation);
		Serializer->Dequantize(Context, MemberArgs);
	}

	// Optional acceleration - we do not need to dequantize it if it isn't replicated
	if (Source.RepAcceleration)
	{
		const FNetSerializer* Serializer = VectorNetQuantizeNetSerializers[Source.VelocityQuantizationLevel];
		const FNetSerializerConfig* SerializerConfig = VectorNetQuantizeNetSerializerConfigs[Source.VelocityQuantizationLevel];

		FNetDequantizeArgs MemberArgs = Args;
		MemberArgs.NetSerializerConfig = NetSerializerConfigParam(SerializerConfig);
		MemberArgs.Source = NetSerializerValuePointer(&Source.Acceleration[0]);
		MemberArgs.Target = NetSerializerValuePointer(&Target.Acceleration);
		Serializer->Dequantize(Context, MemberArgs);
	}

}

bool FRepMovementNetSerializer::IsEqual(FNetSerializationContext& Context, const FNetIsEqualArgs& Args)
{
	if (Args.bStateIsQuantized)
	{
		const QuantizedType& QuantizedValue0 = *reinterpret_cast<const QuantizedType*>(Args.Source0);
		const QuantizedType& QuantizedValue1 = *reinterpret_cast<const QuantizedType*>(Args.Source1);
		return FPlatformMemory::Memcmp(&QuantizedValue0, &QuantizedValue1, sizeof(QuantizedType)) == 0;
	}
	else
	{
		// It's unlikely that any of the flags or quantization levels would differ on the same instance of a FRepMovement struct so we go for the full, expensive, test. 
		QuantizedType QuantizedValue0 = {};
		QuantizedType QuantizedValue1 = {};

		FNetQuantizeArgs QuantizeArgs = {};
		QuantizeArgs.NetSerializerConfig = Args.NetSerializerConfig;

		QuantizeArgs.Source = NetSerializerValuePointer(Args.Source0);
		QuantizeArgs.Target = NetSerializerValuePointer(&QuantizedValue0);
		Quantize(Context, QuantizeArgs);

		QuantizeArgs.Source = NetSerializerValuePointer(Args.Source1);
		QuantizeArgs.Target = NetSerializerValuePointer(&QuantizedValue1);
		Quantize(Context, QuantizeArgs);

		return FPlatformMemory::Memcmp(&QuantizedValue0, &QuantizedValue1, sizeof(QuantizedType)) == 0;
	}
}

bool FRepMovementNetSerializer::Validate(FNetSerializationContext& Context, const FNetValidateArgs& Args)
{
	const SourceType& Value = *reinterpret_cast<SourceType*>(Args.Source);

	// Validate quantization levels
	if ((unsigned(Value.VelocityQuantizationLevel) > unsigned(EVectorQuantization::RoundTwoDecimals))
		| (unsigned(Value.LocationQuantizationLevel) > unsigned(EVectorQuantization::RoundTwoDecimals))
		| (unsigned(Value.RotationQuantizationLevel) > unsigned(ERotatorQuantization::ShortComponents)))
	{
		return false;
	}

	if (Value.bRepPhysics)
	{
		// We expect/know vectors are only checked for NaNs. Let's avoid doing expensive args setup and NetSerializer calls.
		if (Value.AngularVelocity.ContainsNaN())
		{
			return false;
		}
	}

	if (Value.LinearVelocity.ContainsNaN())
	{
		return false;
	}

	if (Value.Location.ContainsNaN())
	{
		return false;
	}

	if (Value.bRepAcceleration)
	{
		if (Value.Acceleration.ContainsNaN())
		{
			return false;
		}
	}

	// Grab an arbitrary RotatorNetSerializer for the rotation validation.
	{
		const FNetSerializer* Serializer = RotatorNetSerializers[0];
		const FNetSerializerConfig* SerializerConfig = RotatorNetSerializerConfigs[0];

		FNetValidateArgs MemberArgs = {};
		MemberArgs.NetSerializerConfig = NetSerializerConfigParam(RotatorNetSerializerConfigs[0]);
		MemberArgs.Source = NetSerializerValuePointer(&Value.Rotation);
		if (!Serializer->Validate(Context, MemberArgs))
		{
			return false;
		}
	}

	return true;
}

void FRepMovementNetSerializer::Apply(FNetSerializationContext& Context, const FNetApplyArgs& Args)
{
	const SourceType& Source = *reinterpret_cast<const SourceType*>(Args.Source);
	SourceType& Target = *reinterpret_cast<SourceType*>(Args.Target);

	Target.bSimulatedPhysicSleep = Source.bSimulatedPhysicSleep;
	Target.bRepPhysics = Source.bRepPhysics;
	Target.ServerFrame = Source.ServerFrame;
	Target.bRepAcceleration = Source.bRepAcceleration;

	// FRepMovement::NetSerialize does not overwrite ServerPhysicsHandle if it wasn't replicated. Despite the member name we mimic the behavior.
	if (Source.ServerPhysicsHandle != INDEX_NONE)
	{
		Target.ServerPhysicsHandle = Source.ServerPhysicsHandle;
	}

	Target.LinearVelocity = Source.LinearVelocity;
	Target.Location = Source.Location;
	Target.Rotation = Source.Rotation;

	// Explicitly do not overwrite AngularVelocity unless it was replicated.
	if (Source.bRepPhysics)
	{
		Target.AngularVelocity = Source.AngularVelocity;
	}

	// Explicitly do not overwrite Acceleration unless it was replicated.
	if (Source.bRepAcceleration)
	{	
		Target.Acceleration = Source.Acceleration;
	}

	// Explicitly do not overwrite quantization levels.
}

// FNetSerializerRegistryDelegates
FRepMovementNetSerializer::FNetSerializerRegistryDelegates::~FNetSerializerRegistryDelegates()
{
	UE_NET_UNREGISTER_NETSERIALIZER_INFO(RepMovementNetSerializer);
}

void FRepMovementNetSerializer::FNetSerializerRegistryDelegates::OnPreFreezeNetSerializerRegistry()
{
	// If our quantized type doesn't meet the requirements of the serializers we're forwarding too then bail out.
	if (!QuantizedTypeMeetRequirements())
	{

		UE_LOGF(LogIris, Error, "FRepMovementNetSerializer will not be registered");
		return;
	}

	if (!IsRepMovementLayoutAsExpected())
	{
		UE_LOGF(LogIris, Error, "FRepMovement layout is unexpected. Things may not replicate as desired.");
	}

	InitNetSerializer();
	UE_NET_REGISTER_NETSERIALIZER_INFO(RepMovementNetSerializer);
}

bool FRepMovementNetSerializer::FNetSerializerRegistryDelegates::QuantizedTypeMeetRequirements() const
{
	// Check vector serializer requirements.
	{
		const FNetSerializer& VectorNetSerializer = UE_NET_GET_SERIALIZER(FVectorNetQuantize100NetSerializer);

		if (!ensure(sizeof(QuantizedType::AngularVelocity) >= VectorNetSerializer.QuantizedTypeSize && alignof(decltype(QuantizedType::AngularVelocity)) >= VectorNetSerializer.QuantizedTypeAlignment))
		{
			return false;
		}

		if (!ensure(sizeof(QuantizedType::LinearVelocity) >= VectorNetSerializer.QuantizedTypeSize && alignof(decltype(QuantizedType::LinearVelocity)) >= VectorNetSerializer.QuantizedTypeAlignment))
		{
			return false;
		}

		if (!ensure(sizeof(QuantizedType::Location) >= VectorNetSerializer.QuantizedTypeSize && alignof(decltype(QuantizedType::Location)) >= VectorNetSerializer.QuantizedTypeAlignment))
		{
			return false;
		}

		if (!ensure(sizeof(QuantizedType::Acceleration) >= VectorNetSerializer.QuantizedTypeSize && alignof(decltype(QuantizedType::Acceleration)) >= VectorNetSerializer.QuantizedTypeAlignment))
		{
			return false;
		}
	}

	// Check rotator serializer requirements.
	{
		const FNetSerializer& RotatorNetSerializer = UE_NET_GET_SERIALIZER(FRotatorAsShortNetSerializer);

		if (!ensure(sizeof(QuantizedType::Rotation) >= RotatorNetSerializer.QuantizedTypeSize && alignof(decltype(QuantizedType::Rotation)) >= RotatorNetSerializer.QuantizedTypeAlignment))
		{
			return false;
		}
	}

	return true;
}

bool FRepMovementNetSerializer::FNetSerializerRegistryDelegates::IsRepMovementLayoutAsExpected() const
{
	const UStruct* RepMovementStruct = FRepMovement::StaticStruct();

	constexpr int ExpectedStructureSize = 136;
	if (!ensureMsgf(RepMovementStruct->GetStructureSize() == ExpectedStructureSize, TEXT("Unexpected FRepMovement structure size. %d != %d"), RepMovementStruct->GetStructureSize(), ExpectedStructureSize))
	{
		return false;
	}

#if !UE_BUILD_SHIPPING
	{
		constexpr const char* Fields[] =
		{
			"LinearVelocity", "AngularVelocity", "Location", "Rotation", "bSimulatedPhysicSleep", "bRepPhysics", 
			"LocationQuantizationLevel", "VelocityQuantizationLevel", "RotationQuantizationLevel", 
			"ServerFrame", "ServerPhysicsHandle", "bRepAcceleration", "Acceleration"
		};
		constexpr SIZE_T FieldCount = sizeof(Fields)/sizeof(Fields[0]);

		for (const char* PropertyName : Fields)
		{
			const FName PropertyFName(PropertyName);
			if (!ensureMsgf(RepMovementStruct->FindPropertyByName(PropertyFName) != nullptr, TEXT("Couldn't find property %s in FRepMovement."), ToCStr(PropertyFName.ToString())))
			{
				return false;
			}
		}
	}
#endif

	// All checks have passed. We believe we can handle the layout of the struct.
	return true;
}

void FRepMovementNetSerializer::FNetSerializerRegistryDelegates::InitNetSerializer()
{
	static_assert(uint8(EVectorQuantization::RoundWholeNumber) == 0U && uint8(EVectorQuantization::RoundOneDecimal) == 1U && uint8(EVectorQuantization::RoundTwoDecimals) == 2U);
	static_assert(uint8(ERotatorQuantization::ByteComponents) == 0U && uint8(ERotatorQuantization::ShortComponents) == 1U);

	FRepMovementNetSerializer::VectorNetQuantizeNetSerializers[0] = &UE_NET_GET_SERIALIZER(FVectorNetQuantizeNetSerializer);
	FRepMovementNetSerializer::VectorNetQuantizeNetSerializers[1] = &UE_NET_GET_SERIALIZER(FVectorNetQuantize10NetSerializer);
	FRepMovementNetSerializer::VectorNetQuantizeNetSerializers[2] = &UE_NET_GET_SERIALIZER(FVectorNetQuantize100NetSerializer);

	FRepMovementNetSerializer::RotatorNetSerializers[0] = &UE_NET_GET_SERIALIZER(FRotatorAsByteNetSerializer);
	FRepMovementNetSerializer::RotatorNetSerializers[1] = &UE_NET_GET_SERIALIZER(FRotatorAsShortNetSerializer);
}

}

