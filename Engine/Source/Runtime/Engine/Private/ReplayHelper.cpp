// Copyright Epic Games, Inc. All Rights Reserved.

#include "ReplayHelper.h"
#include "Engine/ActorChannel.h"
#include "Engine/LevelStreaming.h"
#include "Engine/LevelStreamingDynamic.h"
#include "Engine/LocalPlayer.h"
#include "Engine/NetworkObjectList.h"
#include "GameFramework/GameStateBase.h"
#include "GameFramework/PlayerStart.h"
#include "GameFramework/PlayerState.h"
#include "Misc/App.h"
#include "Net/Core/PropertyConditions/RepChangedPropertyTracker.h"
#include "Net/DataReplication.h"
#include "Net/NetworkGranularMemoryLogging.h"
#include "Net/UnrealNetwork.h"
#include "UnrealEngine.h"
#include "EngineUtils.h"
#include "ReplayNetConnection.h"
#include "Engine/DemoNetDriver.h"
#include "UObject/Package.h"

extern TAutoConsoleVariable<int32> CVarWithLevelStreamingFixes;
extern TAutoConsoleVariable<int32> CVarWithDeltaCheckpoints;
extern TAutoConsoleVariable<int32> CVarWithGameSpecificFrameData;
extern TAutoConsoleVariable<int32> CVarEnableCheckpoints;
extern TAutoConsoleVariable<float> CVarCheckpointUploadDelayInSeconds;
extern TAutoConsoleVariable<float> CVarCheckpointSaveMaxMSPerFrameOverride;
extern TAutoConsoleVariable<int32> CVarDemoUseNetRelevancy;
extern TAutoConsoleVariable<int32> CVarDemoClientRecordAsyncEndOfFrame;
extern TAutoConsoleVariable<float> CVarDemoRecordHz;
extern TAutoConsoleVariable<float> CVarDemoMinRecordHz;

CSV_DECLARE_CATEGORY_EXTERN(Demo);

FReplayHelper::FReplayHelper()
	: CurrentLevelIndex(0)
	, DemoFrameNum(0)
	, DemoCurrentTime(0.0f)
	, DemoTotalTime(0.0f)
	, LastCheckpointTime(0.0)
	, LatestReadFrameTime(0)
	, bWasStartRecordingSuccessful(false)
	, bIsWaitingForStream(false)
	, bIsLoadingCheckpoint(false)
	, bHasLevelStreamingFixes(false)
	, bHasDeltaCheckpoints(false)
	, bHasGameSpecificFrameData(false)
	, bPauseRecording(false)
	, bRecordMapChanges(false)
	, CheckpointSaveMaxMSPerFrame(0)
	, NumLevelsAddedThisFrame(0)
	, bPendingCheckpointRequest(false)
	, bRecording(false)
{
}

FReplayHelper::~FReplayHelper()
{
	FWorldDelegates::LevelRemovedFromWorld.RemoveAll(this);
	FWorldDelegates::LevelAddedToWorld.RemoveAll(this);
}

TSharedPtr<INetworkReplayStreamer> FReplayHelper::Init(const FURL& URL)
{
	DemoURL = URL;

	const TCHAR* const StreamerOverride = DemoURL.GetOption(TEXT("ReplayStreamerOverride="), nullptr);
	ReplayStreamer = FNetworkReplayStreaming::Get().GetFactory(StreamerOverride).CreateReplayStreamer();

	if (ReplayStreamer.IsValid())
	{
		ReplayStreamer->SetAnalyticsProvider(AnalyticsProvider);

		const TCHAR* const DemoPath = DemoURL.GetOption(TEXT("ReplayStreamerDemoPath="), nullptr);
		if (DemoPath != nullptr)
		{
			ReplayStreamer->SetDemoPath(DemoPath);
		}
	}

	TUniquePtr<FReplayResultHandler> ReplayHandler = MakeUnique<FReplayResultHandler>();
	ReplayHandler->InitResultHandler(this);

	ResultManager.AddResultHandler(MoveTemp(ReplayHandler), UE::Net::EAddResultHandlerPos::First);

	FWorldDelegates::LevelRemovedFromWorld.AddRaw(this, &FReplayHelper::OnLevelRemovedFromWorld);
	FWorldDelegates::LevelAddedToWorld.AddRaw(this, &FReplayHelper::OnLevelAddedToWorld);

	if (DemoURL.HasOption(TEXT("CheckpointSaveMaxMSPerFrame")))
	{
		CheckpointSaveMaxMSPerFrame = FCString::Atof(DemoURL.GetOption(TEXT("CheckpointSaveMaxMSPerFrame="), nullptr));
	}

	return ReplayStreamer;
}

void FReplayHelper::SetPlaybackNetworkVersions(FArchive& Ar)
{
	Ar.SetEngineNetVer(PlaybackDemoHeader.GetCustomVersion(FEngineNetworkCustomVersion::Guid));
	Ar.SetGameNetVer(PlaybackDemoHeader.GetCustomVersion(FGameNetworkCustomVersion::Guid));

	Ar.SetUEVer(PlaybackDemoHeader.PackageVersionUE);
	Ar.SetLicenseeUEVer(PlaybackDemoHeader.PackageVersionLicenseeUE);
	// Base archives only store FEngineVersionBase, but the header stores FEngineVersion.
	// This will slice off the branch name and anything else stored in FEngineVersion.
	Ar.SetEngineVer(PlaybackDemoHeader.EngineVersion);

	Ar.SetCustomVersions(PlaybackDemoHeader.CustomVersions);
}

void FReplayHelper::SetPlaybackNetworkVersions(UNetConnection* Connection)
{
	if (Connection)
	{
		Connection->SetNetworkCustomVersions(PlaybackDemoHeader.CustomVersions);
		Connection->SetPackageVersionUE(PlaybackDemoHeader.PackageVersionUE);
		Connection->SetPackageVersionLicenseeUE(PlaybackDemoHeader.PackageVersionLicenseeUE);
		Connection->SetEngineVersion(PlaybackDemoHeader.EngineVersion);
	}
}

FReplayCustomVersion::Type FReplayHelper::GetPlaybackReplayVersion() const
{
	return (FReplayCustomVersion::Type)PlaybackDemoHeader.GetCustomVersion(FReplayCustomVersion::Guid);
}

void FReplayHelper::OnStartRecordingComplete(const FStartStreamingResult& Result)
{
	checkf(Result.bRecording, TEXT("FReplayRecordHelper::StartRecordingComplete: bRecording is false. You may need to enable localReplay.UsePlayerAccountIdPostfix."));

	bIsWaitingForStream = false;
	bWasStartRecordingSuccessful = Result.WasSuccessful();

	if (!bWasStartRecordingSuccessful)
	{
		UE_LOGF(LogDemo, Warning, "FReplayRecordHelper::StartRecordingComplete: Failed");
		NotifyReplayError(EReplayResult::StreamerError);
		return;
	}

	// Broadcast the replay id for anyone interested as by this point it has been finalized.
	// Eg. The replay server telling us the replay id in the case of FHttpNetworkReplayStreamer
	if (ReplayStreamer.IsValid())
	{
		const FString ReplayID = ReplayStreamer->GetReplayID();

		FNetworkReplayDelegates::OnReplayIDChanged.Broadcast(World.Get(), ReplayID);
	}
}

void FReplayHelper::StartRecording(UNetConnection* Connection)
{
	World = Connection ? Connection->GetWorld() : nullptr;

	// World Partition needs bHasLevelStreamingFixes to be true
	bHasLevelStreamingFixes = !!CVarWithLevelStreamingFixes.GetValueOnAnyThread() || World->IsPartitionedWorld();
	bHasDeltaCheckpoints = !!CVarWithDeltaCheckpoints.GetValueOnAnyThread() && ReplayStreamer->IsCheckpointTypeSupported(EReplayCheckpointType::Delta);
	bHasGameSpecificFrameData = !!CVarWithGameSpecificFrameData.GetValueOnAnyThread();

	const TCHAR* FriendlyNameOption = DemoURL.GetOption(TEXT("DemoFriendlyName="), nullptr);

	bRecordMapChanges = DemoURL.GetOption(TEXT("RecordMapChanges"), nullptr) != nullptr;

	TArray<int32> UserIndices;
	for (FLocalPlayerIterator It(GEngine, World.Get()); It; ++It)
	{
		if (*It)
		{
			UserIndices.Add(It->GetControllerId());
		}
	}

	bIsWaitingForStream = true;

	ActiveReplayName = DemoURL.Map;

	if (bHasDeltaCheckpoints)
	{
		ResetDeltaCheckpointTracking(Connection);
	}

	FStartStreamingParameters Params;
	Params.CustomName = DemoURL.Map;
	Params.FriendlyName = FriendlyNameOption != nullptr ? FString(FriendlyNameOption) : World->GetMapName();
	Params.DemoURL = DemoURL.ToString();
	Params.UserIndices = MoveTemp(UserIndices);
	Params.bRecord = true;
	Params.ReplayVersion = FNetworkVersion::GetReplayVersion();

	FNetworkReplayDelegates::OnReplayRecordingStartAttempt.Broadcast(World.Get());

	ReplayStreamer->StartStreaming(Params, FStartStreamingCallback::CreateRaw(this, &FReplayHelper::OnStartRecordingComplete));

	AddNewLevel(GetNameSafe(World->GetOuter()));

	WriteNetworkDemoHeader(Connection);
}

void FReplayHelper::StopReplay()
{
	FNetworkReplayDelegates::OnReplayRecordingComplete.Broadcast(World.Get());

	if (ReplayStreamer.IsValid())
	{
		ReplayStreamer->StopStreaming();
	}

	ActiveReplayName.Empty();
}

void FReplayHelper::WriteNetworkDemoHeader(UNetConnection* Connection)
{
	if (FArchive* FileAr = ReplayStreamer->GetHeaderArchive())
	{
		FNetworkDemoHeader DemoHeader;
		DemoHeader.SetDefaultNetworkVersions();

		DemoHeader.LevelNamesAndTimes = LevelNamesAndTimes;

		FNetworkReplayDelegates::OnWriteGameSpecificDemoHeader.Broadcast(DemoHeader.GameSpecificData);

		UWorld* LocalWorld = World.Get();
		if (LocalWorld)
		{
			// intentionally not checking for a demo net driver
			if (LocalWorld->GetNetDriver() != nullptr && !LocalWorld->GetNetDriver()->IsServer())
			{
				DemoHeader.HeaderFlags |= EReplayHeaderFlags::ClientRecorded;

				if (CVarDemoClientRecordAsyncEndOfFrame.GetValueOnAnyThread() > 0)
				{
					DemoHeader.HeaderFlags |= EReplayHeaderFlags::AsyncRecorded;
				}
			}
		}

		if (HasLevelStreamingFixes())
		{
			DemoHeader.HeaderFlags |= EReplayHeaderFlags::HasStreamingFixes;
		}

		if (HasDeltaCheckpoints())
		{
			DemoHeader.HeaderFlags |= EReplayHeaderFlags::DeltaCheckpoints;
		}

		if (HasGameSpecificFrameData())
		{
			DemoHeader.HeaderFlags |= EReplayHeaderFlags::GameSpecificFrameData;
		}

		if (Cast<UReplayNetConnection>(Connection) != nullptr)
		{
			DemoHeader.HeaderFlags |= EReplayHeaderFlags::ReplayConnection;
		}

		if (Connection)
		{
			if (UDemoNetDriver* DemoDriver = Cast<UDemoNetDriver>(Connection->GetDriver()))
			{
				if (DemoDriver->IsActorPrioritizationEnabled())
				{
					DemoHeader.HeaderFlags |= EReplayHeaderFlags::ActorPrioritizationEnabled;
				}

				DemoHeader.FrameLimitInMS = DemoDriver->GetMaxDesiredRecordTimeMS();
			}
		}

		if (CVarDemoUseNetRelevancy.GetValueOnAnyThread() > 0)
		{
			DemoHeader.HeaderFlags |= EReplayHeaderFlags::NetRelevancyEnabled;
		}

		DemoHeader.Guid = FGuid::NewGuid();

		DemoHeader.CheckpointLimitInMS = CheckpointSaveMaxMSPerFrame;

		DemoHeader.MinRecordHz = CVarDemoMinRecordHz.GetValueOnAnyThread();
		DemoHeader.MaxRecordHz = CVarDemoRecordHz.GetValueOnAnyThread();
		
		DemoHeader.Platform = FPlatformProperties::PlatformName();
		DemoHeader.BuildConfig = FApp::GetBuildConfiguration();
		DemoHeader.BuildTarget = FApp::GetBuildTargetType();

		if (FNetworkReplayDelegates::GetOverridableVersionDataForHeaderWrite.IsBound())
		{
			FOverridableReplayVersionData OverridaleReplayVersionData(DemoHeader);
			FNetworkReplayDelegates::GetOverridableVersionDataForHeaderWrite.Execute(OverridaleReplayVersionData);
			OverridaleReplayVersionData.ApplyVersionDataToDemoHeader(DemoHeader);
		}

		// Write the header
		(*FileAr) << DemoHeader;
		FileAr->Flush();
	}
	else
	{
		UE_LOGF(LogDemo, Error, "WriteNetworkDemoHeader: Header archive is invalid.");
	}
}

void FReplayHelper::OnSeamlessTravelStart(UWorld* InWorld, const FString& LevelName, UNetConnection* Connection)
{
	if (World.Get() == InWorld)
	{
		bPauseRecording = true;

		AddNewLevel(LevelName);

		WriteNetworkDemoHeader(Connection);

		if (ReplayStreamer.IsValid())
		{
			ReplayStreamer->RefreshHeader();
		}
	}
}

APlayerController* FReplayHelper::CreateSpectatorController(UNetConnection* Connection)
{
	// Optionally skip spawning the demo spectator if requested via the URL option
	if (DemoURL.HasOption(TEXT("SkipSpawnSpectatorController")))
	{
		return nullptr;
	}

	check(Connection != nullptr);

	// Get the replay spectator controller class from the default game mode object,
	// since the game mode instance isn't replicated to clients of live games.
	AGameStateBase* GameState = World != nullptr ? World->GetGameState() : nullptr;
	TSubclassOf<AGameModeBase> DefaultGameModeClass = GameState != nullptr ? GameState->GameModeClass : nullptr;

	// If we don't have a game mode class from the world, try to get it from the URL option.
	// This may be true on clients who are recording a replay before the game mode class was replicated to them.
	if (DefaultGameModeClass == nullptr)
	{
		const TCHAR* URLGameModeClass = DemoURL.GetOption(TEXT("game="), nullptr);
		if (URLGameModeClass != nullptr)
		{
			UClass* GameModeFromURL = StaticLoadClass(AGameModeBase::StaticClass(), nullptr, URLGameModeClass);
			DefaultGameModeClass = GameModeFromURL;
		}
	}

	AGameModeBase* DefaultGameMode = DefaultGameModeClass.GetDefaultObject();
	UClass* ReplaySpectatorClass = DefaultGameMode != nullptr ? *DefaultGameMode->ReplaySpectatorPlayerControllerClass : nullptr;

	if (ReplaySpectatorClass == nullptr)
	{
		UE_LOGF(LogDemo, Error, "FReplayHelper::CreateDemoPlayerController: Failed to load demo spectator class.");
		return nullptr;
	}

	FActorSpawnParameters SpawnInfo;
	SpawnInfo.ObjectFlags |= RF_Transient;	// We never want these to save into a map

	APlayerController* NewDemoController = World->SpawnActor<APlayerController>(ReplaySpectatorClass, SpawnInfo);

	if (NewDemoController == nullptr)
	{
		UE_LOGF(LogDemo, Error, "FReplayHelper::CreateDemoPlayerController: Failed to spawn demo spectator.");
		return nullptr;
	}

	// Streaming volumes logic must not be affected by replay spectator camera
	NewDemoController->bIsUsingStreamingVolumes = false;
	NewDemoController->bEnableStreamingSource = false;

	if (Connection->GetDriver())
	{
		// Make sure the player controller GetNetDriver returns this driver. Ensures functions that depend on it,
		// such as IsLocalController, work as expected.
		NewDemoController->SetNetDriverName(Connection->GetDriver()->NetDriverName);
	}

	// If the controller doesn't have a player state, we are probably recording on a client.
	// Spawn one manually.
	if (NewDemoController->PlayerState == nullptr && World != nullptr && World->IsRecordingClientReplay())
	{
		NewDemoController->InitPlayerState();
	}

	// Tell the game that we're spectator and not a normal player
	if (NewDemoController->PlayerState)
	{
		NewDemoController->PlayerState->SetIsOnlyASpectator(true);
	}

	if (TActorIterator<APlayerStart> It(World.Get()); It)
	{
		NewDemoController->SetInitialLocationAndRotation(It->GetActorLocation(), It->GetActorRotation());
	}

	NewDemoController->SetReplicates(true);
	NewDemoController->SetAutonomousProxy(true);
	NewDemoController->SetPlayer(Connection);

	return NewDemoController;
}

void FReplayHelper::AddNewLevel(const FString& NewLevelName)
{
	LevelNamesAndTimes.Add(FLevelNameAndTime(UWorld::RemovePIEPrefix(NewLevelName), ReplayStreamer->GetTotalDemoTime()));
}

bool FReplayHelper::ReadPlaybackDemoHeader(FString& Error)
{
	UGameInstance* GameInstance = World->GetGameInstance();

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	PlaybackDemoHeader = FNetworkDemoHeader();
	PlaybackDemoHeader.SetDefaultNetworkVersions();
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	FArchive* FileAr = ReplayStreamer->GetHeaderArchive();
	if (!FileAr)
	{
		Error = FString::Printf(TEXT("Couldn't open demo file %s for reading"), *DemoURL.Map);
		UE_LOGF(LogDemo, Error, "FReplayHelper::ReadPlaybackDemoHeader: %ls", *Error);
		NotifyReplayError(EReplayResult::ReplayNotFound);
		return false;
	}

	// sanity checking for string/array sizes when reading the header
	FileAr->ArIsNetArchive = true;
	FileAr->ArMaxSerializeSize = FReplayHelper::MAX_DEMO_STRING_SERIALIZATION_SIZE;

	(*FileAr) << PlaybackDemoHeader;

	if (FileAr->IsError())
	{
		Error = FString("Demo file is corrupt");
		UE_LOGF(LogDemo, Error, "FReplayHelper::ReadPlaybackDemoHeader: %ls", *Error);
		NotifyReplayError(EReplayResult::Corrupt);
		return false;
	}

	// Check whether or not we need to process streaming level fixes.
	bHasLevelStreamingFixes = EnumHasAnyFlags(PlaybackDemoHeader.HeaderFlags, EReplayHeaderFlags::HasStreamingFixes);
	// Or delta checkpoints
	bHasDeltaCheckpoints = EnumHasAnyFlags(PlaybackDemoHeader.HeaderFlags, EReplayHeaderFlags::DeltaCheckpoints);

	if (HasDeltaCheckpoints() && !ReplayStreamer->IsCheckpointTypeSupported(EReplayCheckpointType::Delta))
	{
		UE_LOGF(LogDemo, Error, "FReplayHelper::ReadPlaybackDemoHeader: Replay has delta checkpoints but streamer does not support them.");
		NotifyReplayError(EReplayResult::UnsupportedCheckpoint);
		return false;
	}

	bHasGameSpecificFrameData = EnumHasAnyFlags(PlaybackDemoHeader.HeaderFlags, EReplayHeaderFlags::GameSpecificFrameData);

	FNetworkReplayDelegates::OnProcessGameSpecificDemoHeader.Broadcast(PlaybackDemoHeader.GameSpecificData, Error);

	if (FNetworkReplayDelegates::GetOverridableVersionDataForHeaderRead.IsBound())
	{
		FOverridableReplayVersionData OverridaleReplayVersionData(PlaybackDemoHeader);
		FNetworkReplayDelegates::GetOverridableVersionDataForHeaderRead.Execute(OverridaleReplayVersionData);
		OverridaleReplayVersionData.ApplyVersionDataToDemoHeader(PlaybackDemoHeader);
	}
	
	if (!Error.IsEmpty())
	{
		UE_LOGF(LogDemo, Error, "FReplayHelper::ReadPlaybackDemoHeader: (Game Specific) %ls", *Error);
		NotifyReplayError(EReplayResult::GameSpecific);
		return false;
	}

	return true;
}

void FReplayHelper::TickRecording(float DeltaSeconds, UNetConnection* Connection)
{
	if (!bWasStartRecordingSuccessful || bIsWaitingForStream)
	{
		// Nothing to do
		return;
	}

	if (UE::Net::EHandleNetResult::Handled == ReplayStreamer->HandleLastError(ResultManager))
	{
		NotifyReplayError(EReplayResult::StreamerError);
		return;
	}

	if (bPauseRecording)
	{
		return;
	}

	FArchive* FileAr = ReplayStreamer->GetStreamingArchive();

	if (FileAr == nullptr)
	{
		UE_LOGF(LogDemo, Error, "FReplayHelper::TickRecording: FileAr == nullptr");
		NotifyReplayError(EReplayResult::MissingArchive);
		return;
	}

	DECLARE_SCOPE_CYCLE_COUNTER(TEXT("Net replay record time"), STAT_ReplayRecordTime, STATGROUP_Net);

	CSV_SCOPED_TIMING_STAT(Demo, DemoRecordTime);

	// DeltaSeconds that is padded in, is unclampd and not time dilated
	DemoCurrentTime += GetClampedDeltaSeconds(World.Get(), DeltaSeconds);

	ReplayStreamer->UpdateTotalDemoTime(GetDemoCurrentTimeInMS());

	if (CheckpointSaveContext.CheckpointSaveState != FReplayHelper::ECheckpointSaveState::Idle)
	{
		// If we're in the middle of saving a checkpoint, then update that now and return
		TickCheckpoint(Connection);
	}
	else
	{
		RecordFrame(DeltaSeconds, Connection);

		// Save a checkpoint if it's time
		if (CVarEnableCheckpoints.GetValueOnAnyThread() == 1)
		{
			check(CheckpointSaveContext.CheckpointSaveState == FReplayHelper::ECheckpointSaveState::Idle);		// We early out above, so this shouldn't be possible

			if (ShouldSaveCheckpoint())
			{
				SaveCheckpoint(Connection);
			}
		}
	}
}

void FReplayHelper::FlushNetChecked(UNetConnection& NetConnection)
{
	DECLARE_SCOPE_CYCLE_COUNTER(TEXT("Replay net flush"), STAT_ReplayFlushNet, STATGROUP_Net);

	NetConnection.FlushNet();
	check(NetConnection.SendBuffer.GetNumBits() == 0);
}

void FReplayHelper::RecordFrame(float DeltaSeconds, UNetConnection* Connection)
{
	FArchive* FileAr = ReplayStreamer->GetStreamingArchive();
	if (FileAr == nullptr)
	{
		return;
	}

	// Mark any new streaming levels, so that they are saved out this frame
	if (!HasLevelStreamingFixes())
	{
		for (ULevelStreaming* StreamingLevel : World->GetStreamingLevels())
		{
			if (StreamingLevel == nullptr || !StreamingLevel->ShouldBeLoaded() || StreamingLevel->ShouldBeAlwaysLoaded())
			{
				continue;
			}

			TWeakObjectPtr<UObject> WeakStreamingLevel;
			WeakStreamingLevel = StreamingLevel;
			if (!UniqueStreamingLevels.Contains(WeakStreamingLevel))
			{
				UniqueStreamingLevels.Add(WeakStreamingLevel);
				NewStreamingLevelsThisFrame.Add(WeakStreamingLevel);
			}
		}
	}

	FlushNetChecked(*Connection);

	WriteDemoFrame(Connection, *FileAr, QueuedDemoPackets, DemoCurrentTime, EWriteDemoFrameFlags::None);
}

void FReplayHelper::SaveCheckpoint(UNetConnection* Connection)
{
	DECLARE_SCOPE_CYCLE_COUNTER(TEXT("SaveCheckpoint time"), STAT_ReplayCheckpointSaveTime, STATGROUP_Net);

	check(ReplayStreamer.IsValid());

	FArchive* CheckpointArchive = ReplayStreamer->GetCheckpointArchive();

	if (CheckpointArchive == nullptr)
	{
		// This doesn't mean error, it means the streamer isn't ready to save checkpoints
		return;
	}

	check(Connection);
	check(CheckpointArchive->TotalSize() == 0);
	check(Connection->SendBuffer.GetNumBits() == 0);
	check(CheckpointSaveContext.CheckpointSaveState == ECheckpointSaveState::Idle);

	UNetDriver* Driver = Connection->GetDriver();
	check(Driver);

	const FNetworkObjectList& NetworkObjectList = Driver->GetNetworkObjectList();

	const bool bDeltaCheckpoint = HasDeltaCheckpoints();

	CSV_SCOPED_TIMING_STAT(Demo, DemoSaveCheckpointTime);

	const FActorChannelMap& ActorChannelMap = Connection->ActorChannelMap();

	if (HasLevelStreamingFixes())
	{
		SCOPED_NAMED_EVENT(FReplayHelper_ReplayLevelSortAndAssign, FColor::Purple);

		struct FStrippedActorInfo
		{
			TWeakObjectPtr<AActor> Actor;
			const UObject* Level;
		};

		const FNetworkObjectList::FNetworkObjectSet& AllObjectsSet = NetworkObjectList.GetAllObjects();

		TArray<FStrippedActorInfo> ActorArray;
		ActorArray.Reserve(ActorChannelMap.Num() + NetworkObjectList.GetNumDormantActorsForConnection(Connection));

		{
			DECLARE_SCOPE_CYCLE_COUNTER(TEXT("Replay actor level sorting time."), STAT_ReplayLevelSorting, STATGROUP_Net);

			for (const TSharedPtr<FNetworkObjectInfo>& NetworkObjectInfo : AllObjectsSet)
			{
				if (NetworkObjectInfo.IsValid())
				{
					if (!bDeltaCheckpoint || NetworkObjectInfo->bDirtyForReplay)
					{
						const TWeakObjectPtr<AActor>& WeakActor = NetworkObjectInfo->WeakActor;

						// If the actor isn't valid we can't safely access its properties nor serialize it.
						if (!WeakActor.IsValid())
						{
							NetworkObjectInfo->bDirtyForReplay = false;
							continue;
						}

						AActor* Actor = NetworkObjectInfo->Actor;

						// check to see if it should replicate at all
						bool bCheckpointActor = ((Actor->GetRemoteRole() != ROLE_None || Actor->GetTearOff()) && (Actor == Connection->PlayerController || Cast<APlayerController>(Actor) == nullptr));
						
						// now look for an open channel
						bCheckpointActor = bCheckpointActor && ActorChannelMap.Contains(WeakActor);

						if (!bCheckpointActor)
						{
							// has it gone dormant?
							bCheckpointActor = (Actor->NetDormancy != DORM_Initial) &&  (NetworkObjectInfo->DormantConnections.Contains(Connection) || NetworkObjectInfo->RecentlyDormantConnections.Contains(Connection));
						}
						
						if (bCheckpointActor)
						{
#if DO_CHECK && !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
							const ULevel* ActorLevel = Actor->GetLevel();
							checkf(Actor->GetOuter() == ActorLevel, TEXT("Outer %p %s != Level %p %s. Actor %s"), Actor->GetOuter(), ToCStr(GetNameSafe(Actor->GetOuter())), ActorLevel, ToCStr(GetNameSafe(ActorLevel)), ToCStr(GetNameSafe(Actor)));
#endif

							ActorArray.Add({ WeakActor, Actor->GetOuter() });
							NetworkObjectInfo->bDirtyForReplay = false;
						}
					}
				}
			}

			// Sort by level			
			ActorArray.Sort([](const FStrippedActorInfo& A, const FStrippedActorInfo& B) { return (B.Level < A.Level); });
		}

		CheckpointSaveContext.PendingCheckpointActors.Reserve(ActorArray.Num());
		CheckpointSaveContext.PendingActorToIndex.Reserve(ActorArray.Num());

		uint32 LevelIt = 0;
		for (int32 CurrentIt = 0, EndIt = ActorArray.Num(); CurrentIt != EndIt; ++LevelIt)
		{
			const UObject* CurrentLevelToIndex = ActorArray[CurrentIt].Level;
			const FLevelStatus& LevelStatus = FindOrAddLevelStatus(*Cast<const ULevel>(CurrentLevelToIndex));

			while (CurrentIt < EndIt && (CurrentLevelToIndex == ActorArray[CurrentIt].Level))
			{
				int32 PendingIndex = CheckpointSaveContext.PendingCheckpointActors.Add({ ActorArray[CurrentIt].Actor, LevelStatus.LevelIndex });
				CheckpointSaveContext.PendingActorToIndex.Add(ActorArray[CurrentIt].Actor, PendingIndex);
				++CurrentIt;
			};
		}
	}
	else
	{
		// Add any actor with a valid channel to the PendingCheckpointActors list
		for (const TSharedPtr<FNetworkObjectInfo>& ObjectInfo : NetworkObjectList.GetAllObjects())
		{
			AActor* Actor = ObjectInfo.Get()->Actor;

			if (!bDeltaCheckpoint || ObjectInfo->bDirtyForReplay)
			{
				bool bCheckpointActor = ActorChannelMap.Contains(Actor);
				if (!bCheckpointActor)
				{
					bCheckpointActor = (Actor->NetDormancy != DORM_Initial) && (ObjectInfo->DormantConnections.Contains(Connection) || ObjectInfo->RecentlyDormantConnections.Contains(Connection));
				}

				if (bCheckpointActor)
				{
					CheckpointSaveContext.PendingCheckpointActors.Add({ Actor, -1 });

					ObjectInfo->bDirtyForReplay = false;
				}
			}
		}
	}

	if (CheckpointSaveContext.PendingCheckpointActors.Num() == 0)
	{
		return;
	}

	UPackageMapClient* PackageMapClient = Cast<UPackageMapClient>(Connection->PackageMap);

	PackageMapClient->SavePackageMapExportAckStatus(CheckpointSaveContext.CheckpointAckState);

	Connection->SetIgnoreReservedChannels(true);
	Connection->SetReserveDestroyedChannels(true);

	// We are now processing checkpoint actors	
	CheckpointSaveContext.CheckpointSaveState = ECheckpointSaveState::ProcessCheckpointActors;
	CheckpointSaveContext.NextAmortizedItem = 0;
	CheckpointSaveContext.TotalCheckpointSaveTimeSeconds = 0;
	CheckpointSaveContext.TotalCheckpointReplicationTimeSeconds = 0;
	CheckpointSaveContext.TotalCheckpointSaveFrames = 0;
	CheckpointSaveContext.TotalCheckpointActors = CheckpointSaveContext.PendingCheckpointActors.Num();
	CheckpointSaveContext.CheckpointDeletedNetStartupActors.Reset();

	LastCheckpointTime = DemoCurrentTime;

	bPendingCheckpointRequest = false;

	if (bDeltaCheckpoint)
	{
		CheckpointSaveContext.DeltaCheckpointData = MoveTemp(RecordingDeltaCheckpointData);
	}
	else
	{
		CheckpointSaveContext.NameTableMap.Empty();
		CheckpointSaveContext.CheckpointDeletedNetStartupActors.Append(RecordingDeletedNetStartupActors);
	}

	UE_LOGF(LogDemo, Log, "Starting checkpoint. Networked Actors: %i", NetworkObjectList.GetAllObjects().Num());

	// Do the first checkpoint tick now if we're not amortizing
	if (GetCheckpointSaveMaxMSPerFrame() <= 0.0f)
	{
		TickCheckpoint(Connection);
	}
}

// Only start execution if a certain percentage remains of the 
static bool inline ShouldExecuteState(const FRepActorsCheckpointParams& Params, double CurrentTime, double RequiredRatioToStart)
{
	const double CheckpointMaxUploadTimePerFrame = Params.CheckpointMaxUploadTimePerFrame;
	if (CheckpointMaxUploadTimePerFrame <= 0.0)
	{
		return true;
	}

	return (1.0 - ((CurrentTime - Params.StartCheckpointTime) / Params.CheckpointMaxUploadTimePerFrame)) > RequiredRatioToStart;
}

void FReplayHelper::ProcessCheckpointActors(UNetConnection* Connection, TArrayView<FPendingCheckPointActor> PendingActors, int32& NextIndex, FRepActorsCheckpointParams& Params)
{
	UPackageMapClient* PackageMapClient = Cast<UPackageMapClient>(Connection->PackageMap);

	// Set package map ack status override in case we export stuff during the checkpoint
	PackageMapClient->OverridePackageMapExportAckStatus(&CheckpointSaveContext.CheckpointAckState);

	Connection->SetReserveDestroyedChannels(false);

	// Save the replicated server time so we can restore it after the checkpoint has been serialized.
	// This preserves the existing behavior and prevents clients from receiving updated server time
	// more often than the normal update rate.
	AGameStateBase* const GameState = World != nullptr ? World->GetGameState() : nullptr;

	const double SavedReplicatedServerTimeSeconds = GameState ? GameState->ReplicatedWorldTimeSecondsDouble : -1.0;

	// Normally AGameStateBase::ReplicatedWorldTimeSecondsDouble is only updated periodically,
	// but we want to make sure it's accurate for the checkpoint.
	if (GameState)
	{
		GameState->UpdateServerTimeSeconds();
	}

	{
		const bool bDeltaCheckpoint = HasDeltaCheckpoints();

		// Re-use the existing connection to record all properties that have changed since channels were first opened
		TGuardValue<EResendAllDataState> ResendAllData(Connection->ResendAllDataState, bDeltaCheckpoint ? EResendAllDataState::SinceCheckpoint : EResendAllDataState::SinceOpen);

		bool bContinue = true;

		while (NextIndex < PendingActors.Num())
		{
			const FPendingCheckPointActor& Current = PendingActors[NextIndex];

			AActor* Actor = Current.Actor.Get();

			++NextIndex;

			if (!ReplicateCheckpointActor(Actor, Connection, Params))
			{
				break;
			}
		}

		if (GameState)
		{
			// Restore the game state's replicated world time
			GameState->ReplicatedWorldTimeSecondsDouble = SavedReplicatedServerTimeSeconds;
		}

		FlushNetChecked(*Connection);
	}

	Connection->SetReserveDestroyedChannels(true);

	// Restore package map ack status
	PackageMapClient->OverridePackageMapExportAckStatus(nullptr);
}

void FReplayHelper::TickCheckpoint(UNetConnection* Connection)
{
	CSV_SCOPED_TIMING_STAT(Demo, DemoRecordCheckpointTime);
	DECLARE_SCOPE_CYCLE_COUNTER(TEXT("SaveCheckpoint time"), STAT_ReplayCheckpointSaveTime, STATGROUP_Net);

	check(Connection);

	if (CheckpointSaveContext.CheckpointSaveState == ECheckpointSaveState::Idle)
	{
		return;
	}

	FArchive* CheckpointArchive = ReplayStreamer->GetCheckpointArchive();

	if (!ensure(CheckpointArchive != nullptr))
	{
		return;
	}

	CSV_SCOPED_TIMING_STAT(Demo, DemoTickCheckpointTime);

	FRepActorsCheckpointParams Params
	{
		FPlatformTime::Seconds(),
		(double)GetCheckpointSaveMaxMSPerFrame() / 1000
	};

	bool bExecuteNextState = true;
	double CurrentTime = Params.StartCheckpointTime;

	{
		FScopedForceUnicodeInArchive ScopedUnicodeSerialization(*CheckpointArchive);

		CheckpointSaveContext.TotalCheckpointSaveFrames++;

		FReplayHelper::FlushNetChecked(*Connection);

		UPackageMapClient* PackageMapClient = Cast<UPackageMapClient>(Connection->PackageMap);

		const bool bDeltaCheckpoint = HasDeltaCheckpoints();

		while (bExecuteNextState && (CheckpointSaveContext.CheckpointSaveState != ECheckpointSaveState::Finalize) && !(Params.CheckpointMaxUploadTimePerFrame > 0 && CurrentTime - Params.StartCheckpointTime > Params.CheckpointMaxUploadTimePerFrame))
		{
			switch (CheckpointSaveContext.CheckpointSaveState)
			{
			case ECheckpointSaveState::ProcessCheckpointActors:
			{
				SCOPED_NAMED_EVENT(FReplayHelper_ProcessCheckpointActors, FColor::Green);

				{
					FCheckpointStepHelper StepHelper(ECheckpointSaveState::ProcessCheckpointActors, Params.StartCheckpointTime, &CheckpointSaveContext.NextAmortizedItem, CheckpointSaveContext.PendingCheckpointActors.Num());
					ProcessCheckpointActors(Connection, CheckpointSaveContext.PendingCheckpointActors, CheckpointSaveContext.NextAmortizedItem, Params);
				}

				// We are done processing for this frame so store the TotalCheckpointSave time here to be true to the old behavior which did not account for the	actual saving time of the check point
				CheckpointSaveContext.TotalCheckpointReplicationTimeSeconds += (FPlatformTime::Seconds() - Params.StartCheckpointTime);

				// if we have replicated all checkpoint actors, move on to the next state
				if (CheckpointSaveContext.NextAmortizedItem == CheckpointSaveContext.PendingCheckpointActors.Num())
				{
					CheckpointSaveContext.PendingCheckpointActors.Empty();
					CheckpointSaveContext.PendingActorToIndex.Empty();
					
					Connection->SetReserveDestroyedChannels(false);
					Connection->SetIgnoreReservedChannels(false);

					CheckpointSaveContext.CheckpointSaveState = ECheckpointSaveState::SerializeDeletedStartupActors;

					CheckpointSaveContext.bWriteCheckpointOffset = HasLevelStreamingFixes();
					if (HasLevelStreamingFixes())
					{
						CheckpointSaveContext.CheckpointOffset = CheckpointArchive->Tell();
						// We will rewrite this offset when we are done saving the checkpoint
						*CheckpointArchive << CheckpointSaveContext.CheckpointOffset;
					}

					*CheckpointArchive << CurrentLevelIndex;

					CheckpointSaveContext.NextAmortizedItem = 0;
					CheckpointSaveContext.CheckpointSaveState = ECheckpointSaveState::SerializeDeletedStartupActors;
				}
			}
			break;

			case ECheckpointSaveState::SerializeDeletedStartupActors:
			{
				SCOPED_NAMED_EVENT(FReplayHelper_SerializeDeletedStartupActors, FColor::Green);

				// Postpone execution of this state if we have used to much of our alloted time, this value can be tweaked based on profiling
				bExecuteNextState = SerializeDeletedStartupActors(Connection, Params, CheckpointArchive);
				if (bExecuteNextState)
				{
					if (bDeltaCheckpoint)
					{
						CheckpointSaveContext.NextAmortizedItem = 0;
						CheckpointSaveContext.CheckpointSaveState = ECheckpointSaveState::SerializeDeltaDynamicDestroyed;
					}
					else
					{
						CheckpointSaveContext.CheckpointSaveState = ECheckpointSaveState::CacheNetGuids;
					}
				}
			}
			break;

			case ECheckpointSaveState::SerializeDeltaDynamicDestroyed:
			{
				SCOPED_NAMED_EVENT(FReplayHelper_SerializeDeltaDynamicDestroyed, FColor::Green);

				// Postpone execution of this state if we have used to much of our alloted time, this value can be tweaked based on profiling
				bExecuteNextState = SerializeDeltaDynamicDestroyed(Connection, Params, CheckpointArchive);
				if (bExecuteNextState)
				{
					CheckpointSaveContext.NextAmortizedItem = 0;
					CheckpointSaveContext.DeltaCheckpointData.ChannelsToClose.GenerateKeyArray(CheckpointSaveContext.DeltaChannelCloseKeys);

					CheckpointSaveContext.CheckpointSaveState = ECheckpointSaveState::SerializeDeltaClosedChannels;
				}
			}
			break;

			case ECheckpointSaveState::SerializeDeltaClosedChannels:
			{
				SCOPED_NAMED_EVENT(FReplayHelper_SerializeDeltaClosedChannels, FColor::Green);

				// Postpone execution of this state if we have used to much of our alloted time, this value can be tweaked based on profiling
				bExecuteNextState = SerializeDeltaClosedChannels(Connection, Params, CheckpointArchive);
				if (bExecuteNextState)
				{
					CheckpointSaveContext.DeltaCheckpointData.RecordingDeletedNetStartupActors.Empty();
					CheckpointSaveContext.DeltaCheckpointData.DestroyedDynamicActors.Empty();
					CheckpointSaveContext.DeltaCheckpointData.ChannelsToClose.Empty();
					CheckpointSaveContext.DeltaChannelCloseKeys.Empty();

					CheckpointSaveContext.CheckpointSaveState = ECheckpointSaveState::CacheNetGuids;
				}
			}
			break;

			case ECheckpointSaveState::CacheNetGuids:
			{
				// Postpone execution of this state if we have used too much of our alloted time, this value can be tweaked based on profiling
				const double RequiredRatioFor_CacheNetGuids = 0.6;
				if ((bExecuteNextState = ShouldExecuteState(Params, CurrentTime, RequiredRatioFor_CacheNetGuids)) == true)
				{
					SCOPED_NAMED_EVENT(FReplayHelper_CacheNetGuids, FColor::Green);

					CacheNetGuids(Connection);
					CheckpointSaveContext.CheckpointSaveState = ECheckpointSaveState::SerializeGuidCache;
				}
			}
			break;

			case ECheckpointSaveState::SerializeGuidCache:
			{
				SCOPED_NAMED_EVENT(FReplayHelper_SerializeGuidCache, FColor::Green);

				// Save the current guid cache
				bExecuteNextState = SerializeGuidCache(Connection, Params, CheckpointArchive);
				if (bExecuteNextState)
				{
					CheckpointSaveContext.CheckpointSaveState = ECheckpointSaveState::SerializeNetFieldExportGroupMap;
				}
			}
			break;

			case ECheckpointSaveState::SerializeNetFieldExportGroupMap:
			{
				// Postpone execution of this state if we have used to much of our alloted time, this value can be tweaked based on profiling
				const double RequiredRatioFor_SerializeNetFieldExportGroupMap = 0.6;
				if ((bExecuteNextState = ShouldExecuteState(Params, CurrentTime, RequiredRatioFor_SerializeNetFieldExportGroupMap)) == true)
				{
					SCOPED_NAMED_EVENT(FReplayHelper_SerializeNetFieldExportGroupMap, FColor::Green);

					// Save the compatible rep layout map
					if (bDeltaCheckpoint)
					{
						PackageMapClient->SerializeNetFieldExportDelta(*CheckpointArchive);
					}
					else
					{
						PackageMapClient->SerializeNetFieldExportGroupMap(*CheckpointArchive);
					}

					CheckpointSaveContext.CheckpointSaveState = ECheckpointSaveState::SerializeDemoFrameFromQueuedDemoPackets;
				}
			}
			break;

			case ECheckpointSaveState::SerializeDemoFrameFromQueuedDemoPackets:
			{
				// Postpone execution of this state if we have used to much of our alloted time, this value can be tweaked based on profiling
				const double RequiredRatioFor_SerializeDemoFrameFromQueuedDemoPackets = 0.8;
				if ((bExecuteNextState = ShouldExecuteState(Params, CurrentTime, RequiredRatioFor_SerializeDemoFrameFromQueuedDemoPackets)) == true)
				{
					SCOPED_NAMED_EVENT(FReplayHelper_SerializeDemoFrameFromQueuedDemoPackets, FColor::Green);

					// Write offset
					if (CheckpointSaveContext.bWriteCheckpointOffset)
					{
						const FArchivePos CurrentPosition = CheckpointArchive->Tell();
						FArchivePos Offset = CurrentPosition - (CheckpointSaveContext.CheckpointOffset + sizeof(FArchivePos));
						CheckpointArchive->Seek(CheckpointSaveContext.CheckpointOffset);
						*CheckpointArchive << Offset;
						CheckpointArchive->Seek(CurrentPosition);
					}

					// Get the size of the guid data saved
					CheckpointSaveContext.GuidCacheSize = CheckpointArchive->TotalSize();

					// This will cause the entire name list to be written out again.
					// Note, WriteDemoFrame will set this to 0 so we guard the value.
					// This is because when checkpoint amortization is enabled, it's possible for new levels to stream
					// in while recording a checkpoint, and we want to make sure those get written out to the normal
					// streaming archive next frame.
					TGuardValue<uint32> NumLevelsAddedThisFrameGuard(NumLevelsAddedThisFrame, AllLevelStatuses.Num());

					// Write out all of the queued up packets generated while saving the checkpoint
					WriteDemoFrame(Connection, *CheckpointArchive, QueuedCheckpointPackets, static_cast<float>(LastCheckpointTime), EWriteDemoFrameFlags::SkipGameSpecific);

					CheckpointSaveContext.CheckpointSaveState = ECheckpointSaveState::Finalize;
				}
			}
			break;

			default:
				break;
			}

			CurrentTime = FPlatformTime::Seconds();
		}
	}

	// accumulate time spent over all checkpoint ticks
	CheckpointSaveContext.TotalCheckpointSaveTimeSeconds += (CurrentTime - Params.StartCheckpointTime);

	if (CheckpointSaveContext.CheckpointSaveState == ECheckpointSaveState::Finalize)
	{
		SCOPED_NAMED_EVENT(FReplayHelper_Finalize, FColor::Green);

		// Get the total checkpoint size
		const int32 TotalCheckpointSize = CheckpointArchive->TotalSize();

		if (CheckpointArchive->TotalSize() > 0)
		{
			ReplayStreamer->FlushCheckpoint(GetLastCheckpointTimeInMS());
		}

		const float TotalCheckpointTimeInMS = CheckpointSaveContext.TotalCheckpointReplicationTimeSeconds * 1000.0f;
		const float TotalCheckpointTimeWithOverheadInMS = CheckpointSaveContext.TotalCheckpointSaveTimeSeconds * 1000.0f;

		UE_LOGF(LogDemo, Log, "Finished checkpoint. Checkpoint Actors: %i, GuidCacheSize: %i, TotalSize: %i, TotalCheckpointSaveFrames: %i, TotalCheckpointTimeInMS: %2.2f, TotalCheckpointTimeWithOverheadInMS: %2.2f", CheckpointSaveContext.TotalCheckpointActors, CheckpointSaveContext.GuidCacheSize, TotalCheckpointSize, CheckpointSaveContext.TotalCheckpointSaveFrames, TotalCheckpointTimeInMS, TotalCheckpointTimeWithOverheadInMS);

		// we are done, out
		CheckpointSaveContext.CheckpointSaveState = ECheckpointSaveState::Idle;
	}
}

// Checkpoint saving step.
// Serialize as many net guids as fit into a single frame (if time boxed) from previously made snapshot
bool FReplayHelper::SerializeGuidCache(UNetConnection* Connection, const FRepActorsCheckpointParams& Params, FArchive* CheckpointArchive)
{
	check(Connection);

	if (CheckpointSaveContext.NextAmortizedItem == 0) // is the first iteration?
	{
		CheckpointSaveContext.NetGuidsCountPos = CheckpointArchive->Tell();
		*CheckpointArchive << CheckpointSaveContext.NextAmortizedItem;
	}

	FCheckpointStepHelper StepHelper(ECheckpointSaveState::SerializeGuidCache, Params.StartCheckpointTime, &CheckpointSaveContext.NextAmortizedItem, CheckpointSaveContext.NetGuidCacheSnapshot.Num());

	const double Deadline = Params.StartCheckpointTime + Params.CheckpointMaxUploadTimePerFrame;

	check(CheckpointSaveContext.NetGuidCacheSnapshot.Num() == 0 || CheckpointSaveContext.NetGuidCacheSnapshot.IsValidIndex(CheckpointSaveContext.NextAmortizedItem));

	while (CheckpointSaveContext.NextAmortizedItem < CheckpointSaveContext.NetGuidCacheSnapshot.Num())
	{
		FNetworkGUID& NetworkGUID = CheckpointSaveContext.NetGuidCacheSnapshot[CheckpointSaveContext.NextAmortizedItem].NetGuid;
		FNetGuidCacheObject& CacheObject = CheckpointSaveContext.NetGuidCacheSnapshot[CheckpointSaveContext.NextAmortizedItem].NetGuidCacheObject;

		const UObject* Object = CacheObject.Object.Get();

		if (Object && (NetworkGUID.IsStatic() || Object->IsNameStableForNetworking()))
		{
			*CheckpointArchive << NetworkGUID;
			*CheckpointArchive << CacheObject.OuterGUID;

			uint32* NametableIndex = CheckpointSaveContext.NameTableMap.Find(CacheObject.PathName);
			if (NametableIndex == nullptr)
			{
				uint8 bExported = 1;
				*CheckpointArchive << bExported;

				FString PathName = CacheObject.PathName.ToString();
				GEngine->NetworkRemapPath(Connection, PathName, false);

				*CheckpointArchive << PathName;

				uint32 TableIndex = CheckpointSaveContext.NameTableMap.Num();
				
				CheckpointSaveContext.NameTableMap.Add(CacheObject.PathName, TableIndex);
			}
			else
			{
				uint8 bExported = 0;
				*CheckpointArchive << bExported;

				uint32 TableIndex = *NametableIndex;

				CheckpointArchive->SerializeIntPacked(TableIndex);
			}

			uint8 Flags = 0;
			Flags |= CacheObject.bNoLoad ? (1 << 0) : 0;
			Flags |= CacheObject.bIgnoreWhenMissing ? (1 << 1) : 0;

			*CheckpointArchive << Flags;

			++CheckpointSaveContext.NumNetGuidsForRecording;
		}

		++CheckpointSaveContext.NextAmortizedItem;

		if (Params.CheckpointMaxUploadTimePerFrame > 0 && (FPlatformTime::Seconds() >= Deadline))
		{
			break;
		}
	}

	const bool bCompleted = (CheckpointSaveContext.NextAmortizedItem == CheckpointSaveContext.NetGuidCacheSnapshot.Num());
	if (bCompleted)
	{
		FArchivePos Pos = CheckpointArchive->Tell();
		CheckpointArchive->Seek(CheckpointSaveContext.NetGuidsCountPos);
		*CheckpointArchive << CheckpointSaveContext.NumNetGuidsForRecording;
		CheckpointArchive->Seek(Pos);
	}

	return bCompleted;
}

bool FReplayHelper::SerializeDeletedStartupActors(UNetConnection* Connection, const FRepActorsCheckpointParams& Params, FArchive* CheckpointArchive)
{
	check(Connection);

	const bool bDeltaCheckpoint = HasDeltaCheckpoints();

	const TArray<FString>& DeletedActors = bDeltaCheckpoint ? CheckpointSaveContext.DeltaCheckpointData.RecordingDeletedNetStartupActors : CheckpointSaveContext.CheckpointDeletedNetStartupActors;

	if (CheckpointSaveContext.NextAmortizedItem == 0)
	{
		int32 DeletedCount = DeletedActors.Num();
		*CheckpointArchive << DeletedCount;
	}

	FCheckpointStepHelper StepHelper(ECheckpointSaveState::SerializeDeletedStartupActors, Params.StartCheckpointTime, &CheckpointSaveContext.NextAmortizedItem, DeletedActors.Num());

	const double Deadline = Params.StartCheckpointTime + Params.CheckpointMaxUploadTimePerFrame;

	check(DeletedActors.Num() == 0 || DeletedActors.IsValidIndex(CheckpointSaveContext.NextAmortizedItem));

	while (CheckpointSaveContext.NextAmortizedItem < DeletedActors.Num())
	{
		FString DeletedActorPath = DeletedActors[CheckpointSaveContext.NextAmortizedItem];

		GEngine->NetworkRemapPath(Connection, DeletedActorPath, false);

		*CheckpointArchive << DeletedActorPath;

		++CheckpointSaveContext.NextAmortizedItem;

		if (Params.CheckpointMaxUploadTimePerFrame > 0 && (FPlatformTime::Seconds() >= Deadline))
		{
			break;
		}
	}

	const bool bCompleted = (CheckpointSaveContext.NextAmortizedItem == DeletedActors.Num());

	return bCompleted;
}

bool FReplayHelper::SerializeDeltaDynamicDestroyed(UNetConnection* Connection, const FRepActorsCheckpointParams& Params, FArchive* CheckpointArchive)
{
	check(Connection);

	int32 TotalCount = CheckpointSaveContext.DeltaCheckpointData.DestroyedDynamicActors.Num();

	if (CheckpointSaveContext.NextAmortizedItem == 0)
	{
		*CheckpointArchive << TotalCount;
	}

	FCheckpointStepHelper StepHelper(ECheckpointSaveState::SerializeDeltaDynamicDestroyed, Params.StartCheckpointTime, &CheckpointSaveContext.NextAmortizedItem, TotalCount);

	const double Deadline = Params.StartCheckpointTime + Params.CheckpointMaxUploadTimePerFrame;

	check(TotalCount == 0 || CheckpointSaveContext.DeltaCheckpointData.DestroyedDynamicActors.IsValidId(FSetElementId::FromInteger(CheckpointSaveContext.NextAmortizedItem)));

	while (CheckpointSaveContext.NextAmortizedItem < TotalCount)
	{
		FNetworkGUID DestroyedActorGUID = CheckpointSaveContext.DeltaCheckpointData.DestroyedDynamicActors[FSetElementId::FromInteger(CheckpointSaveContext.NextAmortizedItem)];
		*CheckpointArchive << DestroyedActorGUID;

		++CheckpointSaveContext.NextAmortizedItem;

		if (Params.CheckpointMaxUploadTimePerFrame > 0 && (FPlatformTime::Seconds() >= Deadline))
		{
			break;
		}
	}

	const bool bCompleted = (CheckpointSaveContext.NextAmortizedItem == TotalCount);

	return bCompleted;
}

bool FReplayHelper::SerializeDeltaClosedChannels(UNetConnection* Connection, const FRepActorsCheckpointParams& Params, FArchive* CheckpointArchive)
{
	check(Connection);

	int32 TotalCount = CheckpointSaveContext.DeltaChannelCloseKeys.Num();

	if (CheckpointSaveContext.NextAmortizedItem == 0)
	{
		*CheckpointArchive << TotalCount;
	}

	FCheckpointStepHelper StepHelper(ECheckpointSaveState::SerializeDeltaClosedChannels, Params.StartCheckpointTime, &CheckpointSaveContext.NextAmortizedItem, TotalCount);

	const double Deadline = Params.StartCheckpointTime + Params.CheckpointMaxUploadTimePerFrame;

	check(TotalCount == 0 || CheckpointSaveContext.DeltaChannelCloseKeys.IsValidIndex(CheckpointSaveContext.NextAmortizedItem));

	while (CheckpointSaveContext.NextAmortizedItem < TotalCount)
	{
		FNetworkGUID CloseGUID = CheckpointSaveContext.DeltaChannelCloseKeys[CheckpointSaveContext.NextAmortizedItem];
		EChannelCloseReason CloseReason = CheckpointSaveContext.DeltaCheckpointData.ChannelsToClose[CloseGUID];

		*CheckpointArchive << CloseGUID;
		*CheckpointArchive << CloseReason;

		++CheckpointSaveContext.NextAmortizedItem;

		if (Params.CheckpointMaxUploadTimePerFrame > 0 && (FPlatformTime::Seconds() >= Deadline))
		{
			break;
		}
	}

	const bool bCompleted = (CheckpointSaveContext.NextAmortizedItem == TotalCount);

	return bCompleted;
}

void FReplayHelper::ResetLevelStatuses()
{
	ClearLevelStreamingState();

	// There are times (e.g., during travel) when we may not have a valid level.
	// This **should never** be called during those times.
	check(World.Get());

	// ResetLevelStatuses should only ever be called before receiving *any* data from the Replay stream,
	// immediately before processing checkpoint data, or after a level transition (in which case no data
	// will be relevant to the new sublevels).
	// In any case, we can just flag these sublevels as ready immediately.
	FindOrAddLevelStatus(*(World->PersistentLevel)).bIsReady = true;

	for (ULevelStreaming* LevelStreaming : World->GetStreamingLevels())
	{
		if (LevelStreaming && LevelStreaming->IsLevelVisible())
		{
			FindOrAddLevelStatus(*LevelStreaming->GetLoadedLevel()).bIsReady = true;
		}
	}
}

void FReplayHelper::ClearLevelMap()
{
	WeakLevelsByName.Reset();
}

void FReplayHelper::ResetLevelMap()
{
	ClearLevelMap();

	check(World.Get());

	WeakLevelsByName.Add(World->PersistentLevel->GetOutermost()->GetFName(), World->PersistentLevel);

	for (ULevelStreaming* LevelStreaming : World->GetStreamingLevels())
	{
		if (LevelStreaming && LevelStreaming->IsLevelVisible())
		{
			if (ULevel* Level = LevelStreaming->GetLoadedLevel())
			{
				WeakLevelsByName.Add(Level->GetOutermost()->GetFName(), Level);
			}
		}
	}
}

void FReplayHelper::WriteDemoFrame(UNetConnection* Connection, FArchive& Ar, TArray<FQueuedDemoPacket>& QueuedPackets, float FrameTime, EWriteDemoFrameFlags Flags)
{
	DECLARE_SCOPE_CYCLE_COUNTER(TEXT("Replay write frame time"), STAT_ReplayWriteDemoFrame, STATGROUP_Net);

	check(Connection);

	Ar << CurrentLevelIndex;

	// Save total absolute demo time in seconds
	Ar << FrameTime;

	Cast<UPackageMapClient>(Connection->PackageMap)->AppendExportData(Ar);

	if (HasLevelStreamingFixes())
	{
		uint32 NumStreamingLevels = AllLevelStatuses.Num();
		Ar.SerializeIntPacked(NumLevelsAddedThisFrame);

		for (uint32 i = NumStreamingLevels - NumLevelsAddedThisFrame; i < NumStreamingLevels; i++)
		{
			Ar << AllLevelStatuses[i].LevelName;
		}

		NumLevelsAddedThisFrame = 0;
	}
	else
	{
		// Save any new streaming levels
		uint32 NumStreamingLevels = NewStreamingLevelsThisFrame.Num();
		Ar.SerializeIntPacked(NumStreamingLevels);

		for (uint32 i = 0; i < NumStreamingLevels; i++)
		{
			ULevelStreaming* StreamingLevel = World->GetStreamingLevels()[i];

			// TODO: StreamingLevel could be null, but since we've already written out the integer count, skipping entries could cause an issue, so leaving as is for now
			FString PackageName = StreamingLevel->GetWorldAssetPackageName();
			FString PackageNameToLoad = StreamingLevel->PackageNameToLoad.ToString();

			Ar << PackageName;
			Ar << PackageNameToLoad;
			Ar << StreamingLevel->LevelTransform;

			UE_LOGF(LogDemo, Log, "WriteDemoFrame: StreamingLevel: %ls, %ls", *PackageName, *PackageNameToLoad);
		}

		NewStreamingLevelsThisFrame.Empty();
	}

	{
		TUniquePtr<FScopedStoreArchiveOffset> ScopedOffset(HasLevelStreamingFixes() ? new FScopedStoreArchiveOffset(Ar) : nullptr);

		// Save external data
		SaveExternalData(Connection, Ar);
	}

	if (HasGameSpecificFrameData())
	{
		FScopedStoreArchiveOffset ScopedOffset(Ar);

		if (!EnumHasAnyFlags(Flags, EWriteDemoFrameFlags::SkipGameSpecific))
		{
			FDemoFrameDataMap Data;
			FNetworkReplayDelegates::OnWriteGameSpecificFrameData.Broadcast(World.Get(), FrameTime, Data);

			Ar << Data;
		}
	}

	for (FQueuedDemoPacket& DemoPacket : QueuedPackets)
	{
		if (HasLevelStreamingFixes())
		{
			ensureAlways(DemoPacket.SeenLevelIndex);
			Ar.SerializeIntPacked(DemoPacket.SeenLevelIndex);
		}

		WritePacket(Ar, DemoPacket.Data.GetData(), DemoPacket.Data.Num());
	}

	QueuedPackets.Empty();

	if (HasLevelStreamingFixes())
	{
		uint32 EndCountUnsigned = 0;
		Ar.SerializeIntPacked(EndCountUnsigned);
	}

	// Write a count of 0 to signal the end of the frame
	int32 EndCount = 0;
	Ar << EndCount;
}

void FReplayHelper::WritePacket(FArchive& Ar, uint8* Data, int32 Count)
{
	Ar << Count;
	Ar.Serialize(Data, Count);
}

void FReplayHelper::SaveExternalData(UNetConnection* Connection, FArchive& Ar)
{
	check(Connection && Connection->Driver);

	SCOPED_NAMED_EVENT(FReplayHelper_SaveExternalData, FColor::Blue);
	for (TPair<TWeakObjectPtr<UObject>, FNetworkGUID>& Pair : ObjectsWithExternalDataMap)
	{
		if (UObject* Object = Pair.Key.Get())
		{
			if (FExternalDataWrapper* ExternalData = ExternalDataMap.Find(Object))
			{
				if (ExternalData->NumBits > 0)
				{
					FNetworkGUID NetworkGUID = ExternalData->NetGUID;
					if (!NetworkGUID.IsValid())
					{
						// try the lookup again, it may not have been registered when the data was added
						NetworkGUID = Connection->Driver->GetNetGuidCache()->NetGUIDLookup.FindRef(Object);
					}

					if (NetworkGUID.IsValid())
					{
						// Save payload size (in bits)
						uint32 NumBits = ExternalData->NumBits;
						Ar.SerializeIntPacked(NumBits);

						// Save GUID
						Ar << NetworkGUID;

						// Save payload
						Ar.Serialize(ExternalData->Data.GetData(), ExternalData->Data.Num());
					}
					else
					{
						UE_LOGF(LogDemo, Warning, "SaveExternalData: Discarding external data for object with no net guid: %ls", *GetNameSafe(Object));
					}
				}
			}
		}
	}

	// Reset external out datas
	ObjectsWithExternalDataMap.Reset();
	ExternalDataMap.Reset();

	uint32 StopCount = 0;
	Ar.SerializeIntPacked(StopCount);
}

FString FReplayHelper::GetLevelPackageName(const ULevel& InLevel)
{
	FString PathName = InLevel.GetOutermost()->GetFName().ToString();
	return UWorld::RemovePIEPrefix(PathName);
}

float FReplayHelper::GetClampedDeltaSeconds(UWorld* World, const float DeltaSeconds)
{
	check(World != nullptr);

	const float RealDeltaSeconds = DeltaSeconds;

	// Clamp delta seconds
	AWorldSettings* WorldSettings = World->GetWorldSettings();
	const float ClampedDeltaSeconds = WorldSettings->FixupDeltaSeconds(DeltaSeconds * WorldSettings->GetEffectiveTimeDilation(), RealDeltaSeconds);
	check(ClampedDeltaSeconds >= 0.0f);

	return ClampedDeltaSeconds;
}

void FReplayHelper::CacheNetGuids(UNetConnection* Connection)
{
	if (Connection && Connection->Driver)
	{
		int32 NumValues = 0;
		const bool bDeltaCheckpoint = HasDeltaCheckpoints();
		const double StartTime = FPlatformTime::Seconds();

		// initialize NetGuidCache serialization
		CheckpointSaveContext.NetGuidCacheSnapshot.Reset();
		CheckpointSaveContext.NextAmortizedItem = 0;
		CheckpointSaveContext.NumNetGuidsForRecording = 0;

		for (auto It = Connection->Driver->GetNetGuidCache()->ObjectLookup.CreateIterator(); It; ++It)
		{
			FNetworkGUID& NetworkGUID = It.Key();
			FNetGuidCacheObject& CacheObject = It.Value();

			if (bDeltaCheckpoint && !CacheObject.bDirtyForReplay)
			{
				continue;
			}

			// Do not add guids we would filter out in the serialize step
			if (NetworkGUID.IsValid() && CacheObject.Object.Get() && (NetworkGUID.IsStatic() || CacheObject.Object->IsNameStableForNetworking()))
			{
				CheckpointSaveContext.NetGuidCacheSnapshot.Add({ NetworkGUID, CacheObject });

				CacheObject.bDirtyForReplay = false;

				++NumValues;
			}
		}

		UE_LOGF(LogDemo, Verbose, "CacheNetGuids: %d, %.1f ms", NumValues, (FPlatformTime::Seconds() - StartTime) * 1000);
	}
}

bool FReplayHelper::ReplicateCheckpointActor(AActor* ToReplicate, UNetConnection* Connection, class FRepActorsCheckpointParams& Params)
{
	// Early out if the actor has been destroyed or the world is streamed out.
	if (ToReplicate == nullptr || ToReplicate->GetWorld() == nullptr)
	{
		return true;
	}

	bool bOpenedChannelForDormancy = false;

	UActorChannel* ActorChannel = Connection->FindActorChannelRef(ToReplicate);

	if (ActorChannel == nullptr && ToReplicate->NetDormancy > DORM_Awake)
	{
		// Create a new channel for this actor.
		ActorChannel = Cast<UActorChannel>(Connection->CreateChannelByName(NAME_Actor, EChannelCreateFlags::OpenedLocally));
		if (ActorChannel != nullptr)
		{
			ActorChannel->SetChannelActor(ToReplicate, ESetChannelActorFlags::SkipMarkActive);

			bOpenedChannelForDormancy = true;
		}
	}

	if (ActorChannel)
	{
		const bool bReplicated = ReplicateActor(ToReplicate, Connection, true);

		if (bOpenedChannelForDormancy)
		{
			ActorChannel->bPendingDormancy = false;
			ActorChannel->bIsInDormancyHysteresis = false;
			ActorChannel->Dormant = true;

			if (bReplicated)
			{
				ActorChannel->Close(EChannelCloseReason::Dormancy);
			}

			ActorChannel->ConditionalCleanUp(false, EChannelCloseReason::Dormancy);
		}

		const double CheckpointTime = FPlatformTime::Seconds();

		if (Params.CheckpointMaxUploadTimePerFrame > 0 && CheckpointTime - Params.StartCheckpointTime > Params.CheckpointMaxUploadTimePerFrame)
		{
			return false;
		}
	}

	return true;
}

void FReplayHelper::LoadExternalData(FArchive& Ar, const float TimeSeconds)
{
	DECLARE_SCOPE_CYCLE_COUNTER(TEXT("Demo_LoadExternalData"), Demo_LoadExternalData, STATGROUP_Net);

	while (true)
	{
		uint32 ExternalDataNumBits;

		// Read payload into payload/guid map
		Ar.SerializeIntPacked(ExternalDataNumBits);

		if (ExternalDataNumBits == 0)
		{
			return;
		}

		FNetworkGUID NetGUID;

		// Read net guid this payload belongs to
		Ar << NetGUID;

		const int64 ExternalDataNumBytes = ((int64)ExternalDataNumBits + 7) >> 3;

		FBitReader Reader(nullptr, ExternalDataNumBits);

		Ar.Serialize(Reader.GetData(), ExternalDataNumBytes);

		SetPlaybackNetworkVersions(Reader);

		FReplayExternalDataArray& ExternalDataArray = ExternalDataToObjectMap.FindOrAdd(NetGUID);

		ExternalDataArray.Add(new FReplayExternalData(MoveTemp(Reader), TimeSeconds));
	}
}

// only allow one chunk of data per object at a time (first wins), when recording ticks it will store/clear it
bool FReplayHelper::SetExternalDataForObject(UNetConnection* Connection, UObject* OwningObject, const uint8* Src, const int32 NumBits)
{
	check(Connection && Connection->Driver);

	// It's fine if we don't find the guid, we may replicate the actor later this frame which will register it
	FNetworkGUID NetworkGUID = Connection->Driver->GetNetGuidCache()->NetGUIDLookup.FindRef(OwningObject);

	if (!ExternalDataMap.Contains(OwningObject))
	{
		ObjectsWithExternalDataMap.Add(OwningObject, NetworkGUID);

		FExternalDataWrapper ExternalData;
		ExternalData.NetGUID = NetworkGUID;
		ExternalData.NumBits = NumBits;

		const int32 NumBytes = (NumBits + 7) >> 3;

		ExternalData.Data.AddUninitialized(NumBytes);
		FMemory::Memcpy(ExternalData.Data.GetData(), Src, NumBytes);

		ExternalDataMap.Emplace(OwningObject, MoveTemp(ExternalData));
		return true;
	}
	else
	{
		UE_LOGF(LogDemo, Warning, "SetExternalDataForObject: Discarding external data for object, already exists: %ls", *GetNameSafe(OwningObject));
	}

	return false;
}

void FReplayHelper::FCheckpointSaveStateContext::CountBytes(FArchive& Ar) const
{
	GRANULAR_NETWORK_MEMORY_TRACKING_INIT(Ar, "FCheckpointSaveStateContext::CountBytes");

	GRANULAR_NETWORK_MEMORY_TRACKING_TRACK("CheckpointAckState", CheckpointAckState.CountBytes(Ar));
	GRANULAR_NETWORK_MEMORY_TRACKING_TRACK("PendingCheckpointActors", PendingCheckpointActors.CountBytes(Ar));
	GRANULAR_NETWORK_MEMORY_TRACKING_TRACK("PendingActorToIndex", PendingActorToIndex.CountBytes(Ar));
	GRANULAR_NETWORK_MEMORY_TRACKING_TRACK("DeltaCheckpointData", DeltaCheckpointData.CountBytes(Ar));
	GRANULAR_NETWORK_MEMORY_TRACKING_TRACK("DeltaChannelCloseKeys", DeltaChannelCloseKeys.CountBytes(Ar));
	GRANULAR_NETWORK_MEMORY_TRACKING_TRACK("NetGuidCacheSnapshot", NetGuidCacheSnapshot.CountBytes(Ar));
	GRANULAR_NETWORK_MEMORY_TRACKING_TRACK("CheckpointDeletedNetStartupActors", CheckpointDeletedNetStartupActors.CountBytes(Ar));
	GRANULAR_NETWORK_MEMORY_TRACKING_TRACK("NameTableMap", NameTableMap.CountBytes(Ar));
}

void FReplayHelper::Serialize(FArchive& Ar)
{
	GRANULAR_NETWORK_MEMORY_TRACKING_INIT(Ar, "FReplayHelper::Serialize");

	if (Ar.IsCountingMemory())
	{
		GRANULAR_NETWORK_MEMORY_TRACKING_TRACK("UniqueStreamingLevels", UniqueStreamingLevels.CountBytes(Ar));
		GRANULAR_NETWORK_MEMORY_TRACKING_TRACK("NewStreamingLevelsThisFrame", NewStreamingLevelsThisFrame.CountBytes(Ar));

		GRANULAR_NETWORK_MEMORY_TRACKING_TRACK("PlaybackDemoHeader", PlaybackDemoHeader.CountBytes(Ar));

		GRANULAR_NETWORK_MEMORY_TRACKING_TRACK("LevelNamesAndTimes",
			LevelNamesAndTimes.CountBytes(Ar);
			for (const FLevelNameAndTime& LevelNameAndTime : LevelNamesAndTimes)
			{
				LevelNameAndTime.CountBytes(Ar);
			}
		);

		GRANULAR_NETWORK_MEMORY_TRACKING_TRACK("AllLevelStatuses",
			AllLevelStatuses.CountBytes(Ar);
			for (const FLevelStatus& LevelStatus : AllLevelStatuses)
			{
				LevelStatus.CountBytes(Ar);
			}
		);

		GRANULAR_NETWORK_MEMORY_TRACKING_TRACK("LevelStatusesByName",
			LevelStatusesByName.CountBytes(Ar);
			for (const auto& LevelStatusNamePair : LevelStatusesByName)
			{
				LevelStatusNamePair.Key.CountBytes(Ar);
			}
		);

		GRANULAR_NETWORK_MEMORY_TRACKING_TRACK("LevelStatusIndexByLevel", LevelStatusIndexByLevel.CountBytes(Ar));
		GRANULAR_NETWORK_MEMORY_TRACKING_TRACK("SeenLevelStatuses", SeenLevelStatuses.CountBytes(Ar));
		GRANULAR_NETWORK_MEMORY_TRACKING_TRACK("LevelsPendingFastForward", LevelsPendingFastForward.CountBytes(Ar));
		GRANULAR_NETWORK_MEMORY_TRACKING_TRACK("ObjectsWithExternalDataMap", ObjectsWithExternalDataMap.CountBytes(Ar));
		GRANULAR_NETWORK_MEMORY_TRACKING_TRACK("ExternalDataMap", ExternalDataMap.CountBytes(Ar));
		GRANULAR_NETWORK_MEMORY_TRACKING_TRACK("CheckpointSaveContext", CheckpointSaveContext.CountBytes(Ar));

		GRANULAR_NETWORK_MEMORY_TRACKING_TRACK("QueuedDemoPackets",
			QueuedDemoPackets.CountBytes(Ar);
			for (const FQueuedDemoPacket& QueuedPacket : QueuedDemoPackets)
			{
				QueuedPacket.CountBytes(Ar);
			}
		);

		GRANULAR_NETWORK_MEMORY_TRACKING_TRACK("QueuedCheckpointPackets",
			QueuedCheckpointPackets.CountBytes(Ar);
			for (const FQueuedDemoPacket& QueuedPacket : QueuedCheckpointPackets)
			{
				QueuedPacket.CountBytes(Ar);
			}
		);

		GRANULAR_NETWORK_MEMORY_TRACKING_TRACK("ExternalDataToObjectMap",
			ExternalDataToObjectMap.CountBytes(Ar);
			for (const auto& ExternalDataToObjectPair : ExternalDataToObjectMap)
			{
				ExternalDataToObjectPair.Value.CountBytes(Ar);
			}
		);

		GRANULAR_NETWORK_MEMORY_TRACKING_TRACK("PlaybackFrames",
			PlaybackFrames.CountBytes(Ar);
			for (auto& Frame : PlaybackFrames)
			{
				Frame.Value.CountBytes(Ar);
			}
		);

		GRANULAR_NETWORK_MEMORY_TRACKING_TRACK("RecordingDeletedNetStartupActors",
			RecordingDeletedNetStartupActors.CountBytes(Ar);
			for (FString& ActorString : RecordingDeletedNetStartupActors)
			{
				Ar << ActorString;
			}
		);

		GRANULAR_NETWORK_MEMORY_TRACKING_TRACK("RecordingDeletedNetStartupActors",
			RecordingDeletedNetStartupActors.CountBytes(Ar);
			for (FString& ActorString : RecordingDeletedNetStartupActors)
			{
				Ar << ActorString;
			}
		);

		GRANULAR_NETWORK_MEMORY_TRACKING_TRACK("WeakLevelsByName", WeakLevelsByName.CountBytes(Ar));
	}
}

bool FReplayHelper::ReplicateActor(AActor* Actor, UNetConnection* Connection, bool bMustReplicate)
{
	if (UNetDriver::IsDormInitialStartupActor(Actor))
	{
		return false;
	}

	check(Connection && Connection->Driver);

	const int32 OriginalOutBunches = Connection->Driver->OutBunches;

	bool bDidReplicateActor = false;

	Actor->CallPreReplication(Connection->Driver);

	// Handle role swapping if this is a client-recorded replay.
	FScopedActorRoleSwap RoleSwap(Actor);

	if ((Actor->GetRemoteRole() != ROLE_None || Actor->GetTearOff()) && (Actor == Connection->PlayerController || Cast<APlayerController>(Actor) == nullptr))
	{
		const bool bShouldHaveChannel =
			Actor->bRelevantForNetworkReplays &&
			!Actor->GetTearOff() &&
			(!Actor->IsNetStartupActor() || Connection->ClientHasInitializedLevel(Actor->GetLevel()));

		UActorChannel* Channel = Connection->FindActorChannelRef(Actor);

		if (bShouldHaveChannel && Channel == nullptr)
		{
			// Create a new channel for this actor.
			Channel = (UActorChannel*)Connection->CreateChannelByName(NAME_Actor, EChannelCreateFlags::OpenedLocally);
			if (Channel != nullptr)
			{
				Channel->SetChannelActor(Actor, ESetChannelActorFlags::None);
			}
		}

		if (Channel != nullptr && !Channel->Closing)
		{
			// Send it out!
			bDidReplicateActor = (Channel->ReplicateActor() > 0);

			// Close the channel if this actor shouldn't have one
			if (!bShouldHaveChannel)
			{
				if (Connection->ResendAllDataState == EResendAllDataState::None)		// Don't close the channel if we're forcing them to re-open for checkpoints
				{
					Channel->Close(Actor->GetTearOff() ? EChannelCloseReason::TearOff : EChannelCloseReason::Destroyed);
				}
			}
		}
	}

	if (bMustReplicate && !HasDeltaCheckpoints() && Connection->Driver->OutBunches == OriginalOutBunches)
	{
		UE_LOGF(LogDemo, Error, "DemoReplicateActor: bMustReplicate is true but nothing was sent: %ls", *GetNameSafe(Actor));
	}

	return bDidReplicateActor;
}

bool FReplayHelper::ReadDemoFrame(UNetConnection* Connection, FArchive& Ar, TArray<FPlaybackPacket>& InPlaybackPackets, const bool bForLevelFastForward, const FArchivePos MaxArchiveReadPos, float* OutTime)
{
	SCOPED_NAMED_EVENT(FReplayHelper_ReadDemoFrame, FColor::Purple);

	DECLARE_SCOPE_CYCLE_COUNTER(TEXT("ReadDemoFrame"), ReadDemoFrame, STATGROUP_Net);

	check(!bForLevelFastForward || HasLevelStreamingFixes());
	check(Connection);

	if (Ar.IsError())
	{
		UE_LOGF(LogDemo, Error, "FReplayHelper::ReadDemoFrame: Archive Error");
		NotifyReplayError(EReplayResult::Serialization);
		return false;
	}

	if (Ar.AtEnd())
	{
		return false;
	}

	if (UE::Net::EHandleNetResult::Handled == ReplayStreamer->HandleLastError(ResultManager))
	{
		NotifyReplayError(EReplayResult::StreamerError);
		return false;
	}

	// Above checks guarantee the Archive is in a valid state, but it's entirely possible that
	// the ReplayStreamer doesn't have more stream data available (i.e., if we only have checkpoint data).
	// Therefore, skip this if we know we're only reading in checkpoint data.
	if (!bIsLoadingCheckpoint && !ReplayStreamer->IsDataAvailable())
	{
		return false;
	}

	int32 ReadCurrentLevelIndex = 0;

	Ar << ReadCurrentLevelIndex;

	float TimeSeconds = 0.0f;

	Ar << TimeSeconds;

	if (OutTime)
	{
		*OutTime = TimeSeconds;
	}

	{
		DECLARE_SCOPE_CYCLE_COUNTER(TEXT("Demo_ReceiveExports"), Demo_ReceiveExports, STATGROUP_Net);
		CastChecked<UPackageMapClient>(Connection->PackageMap)->ReceiveExportData(Ar);
	}

	// Check to see if we can skip adding these packets.
	// This may happen if the archive isn't set to a proper position due to level fast forwarding.
	const bool bAppendPackets = bIsLoadingCheckpoint || bForLevelFastForward || LatestReadFrameTime < TimeSeconds;

	LatestReadFrameTime = FMath::Max(LatestReadFrameTime, TimeSeconds);

	if (HasLevelStreamingFixes())
	{
		uint32 NumStreamingLevels = 0;
		Ar.SerializeIntPacked(NumStreamingLevels);

		// We want to avoid adding the same levels to the Seen list multiple times.
		// This can occur if the Archive is "double read" due to a level fast forward.
		const bool bAddToSeenList = bAppendPackets && !bForLevelFastForward;

		for (uint32 i = 0; i < NumStreamingLevels; i++)
		{
			FString NameTemp;
			Ar << NameTemp;

			if (bAddToSeenList)
			{
				// Add this level to the seen list, but don't actually mark it as being seen.
				// It will be marked when we have processed packets for it.
				const FLevelStatus& LevelStatus = FindOrAddLevelStatus(MoveTemp(NameTemp));
				SeenLevelStatuses.Add(LevelStatus.LevelIndex);
			}
		}
	}
	else
	{
		// Read any new streaming levels this frame
		uint32 NumStreamingLevels = 0;
		Ar.SerializeIntPacked(NumStreamingLevels);

		for (uint32 i = 0; i < NumStreamingLevels; ++i)
		{
			FString PackageName;
			FString PackageNameToLoad;
			FTransform LevelTransform;

			Ar << PackageName;
			Ar << PackageNameToLoad;
			Ar << LevelTransform;

			// Don't add if already exists
			bool bFound = false;

			for (ULevelStreaming* StreamingLevel : World->GetStreamingLevels())
			{
				FString SrcPackageName = StreamingLevel->GetWorldAssetPackageName();
				FString SrcPackageNameToLoad = StreamingLevel->PackageNameToLoad.ToString();

				if (SrcPackageName == PackageName && SrcPackageNameToLoad == PackageNameToLoad)
				{
					bFound = true;
					break;
				}
			}

			if (bFound)
			{
				continue;
			}

			ULevelStreamingDynamic* StreamingLevel = NewObject<ULevelStreamingDynamic>(World.Get(), NAME_None, RF_NoFlags, nullptr);

			StreamingLevel->SetShouldBeLoaded(true);
			StreamingLevel->SetShouldBeVisible(true);
			StreamingLevel->bShouldBlockOnLoad = false;
			StreamingLevel->bInitiallyLoaded = true;
			StreamingLevel->bInitiallyVisible = true;
			StreamingLevel->LevelTransform = LevelTransform;

			StreamingLevel->PackageNameToLoad = FName(*PackageNameToLoad);
			StreamingLevel->SetWorldAssetByPackageName(FName(*PackageName));

			World->AddStreamingLevel(StreamingLevel);

			UE_LOGF(LogDemo, Log, "ReadDemoFrame: Loading streamingLevel: %ls, %ls", *PackageName, *PackageNameToLoad);
		}
	}

	if (Ar.IsError())
	{
		UE_LOGF(LogDemo, Error, "FReplayHelper::ReadDemoFrame: Failed to read demo ServerDeltaTime");
		NotifyReplayError(EReplayResult::Serialization);
		return false;
	}

	FArchivePos SkipExternalOffset = 0;
	if (HasLevelStreamingFixes())
	{
		Ar << SkipExternalOffset;
	}

	if (!bForLevelFastForward)
	{
		// Load any custom external data in this frame
		LoadExternalData(Ar, TimeSeconds);
	}
	else
	{
		Ar.Seek(Ar.Tell() + SkipExternalOffset);
	}

	FArchivePos SkipGameSpecificOffset = 0;
	if (HasGameSpecificFrameData())
	{
		Ar << SkipGameSpecificOffset;

		if ((SkipGameSpecificOffset > 0) && !bForLevelFastForward)
		{
			FDemoFrameDataMap Data;
			Ar << Data;

			if (Data.Num() > 0)
			{
				PlaybackFrames.Emplace(TimeSeconds, MoveTemp(Data));
			}
		}
		else
		{
			Ar.Seek(Ar.Tell() + SkipGameSpecificOffset);
		}
	}

	{
		DECLARE_SCOPE_CYCLE_COUNTER(TEXT("Demo_ReadPackets"), Demo_ReadPackets, STATGROUP_Net);

		FPlaybackPacket ScratchPacket;
		ScratchPacket.TimeSeconds = TimeSeconds;
		ScratchPacket.LevelIndex = ReadCurrentLevelIndex;
		ScratchPacket.SeenLevelIndex = INDEX_NONE;

		const EReadPacketMode ReadPacketMode = bAppendPackets ? EReadPacketMode::Default : EReadPacketMode::SkipData;

		while ((MaxArchiveReadPos == 0) || (Ar.Tell() < MaxArchiveReadPos))
		{
			if (HasLevelStreamingFixes())
			{
				Ar.SerializeIntPacked(ScratchPacket.SeenLevelIndex);
			}

			switch (ReadPacket(Ar, ScratchPacket.Data, ReadPacketMode))
			{
			case EReadPacketState::Error:
			{
				UE_LOGF(LogDemo, Error, "FReplayHelper::ReadDemoFrame: ReadPacket failed.");
				NotifyReplayError(EReplayResult::Serialization);
				return false;
			}

			case EReadPacketState::Success:
			{
				if (EReadPacketMode::SkipData == ReadPacketMode)
				{
					continue;
				}

				InPlaybackPackets.Emplace(MoveTemp(ScratchPacket));
				ScratchPacket.Data = TArray<uint8>();
				break;
			}

			case EReadPacketState::End:
			{
				return true;
			}

			default:
			{
				check(false);
				return false;
			}
			}
		}
	}

	// We should never hit this, as the while loop above should return on error or success.
	check(false);
	return false;
}

const FReplayHelper::EReadPacketState FReplayHelper::ReadPacket(FArchive& Archive, TArray<uint8>& OutBuffer, const EReadPacketMode Mode)
{
	const bool bSkipData = (EReadPacketMode::SkipData == Mode);

	int32 BufferSize = 0;
	Archive << BufferSize;

	if (UNLIKELY(Archive.IsError()))
	{
		UE_LOGF(LogDemo, Error, "FReplayHelper::ReadPacket: Failed to read demo OutBufferSize");
		return EReadPacketState::Error;
	}

	if (BufferSize == 0)
	{
		return EReadPacketState::End;
	}

	else if (UNLIKELY(BufferSize > FReplayHelper::MAX_DEMO_READ_WRITE_BUFFER))
	{
		UE_LOG(LogDemo, Error, TEXT("FReplayHelper::ReadPacket: OutBufferSize > MAX_DEMO_READ_WRITE_BUFFER"));
		return EReadPacketState::Error;
	}
	else if (UNLIKELY(BufferSize < 0))
	{
		UE_LOGF(LogDemo, Error, "FReplayHelper::ReadPacket: OutBufferSize < 0");
		return EReadPacketState::Error;
	}

	if (bSkipData)
	{
		Archive.Seek(Archive.Tell() + static_cast<int64>(BufferSize));
	}
	else
	{
		OutBuffer.SetNumUninitialized(BufferSize, EAllowShrinking::No);
		Archive.Serialize(OutBuffer.GetData(), BufferSize);
	}

	if (UNLIKELY(Archive.IsError()))
	{
		UE_LOGF(LogDemo, Error, "FReplayHelper::ReadPacket: Failed to read demo file packet");
		return EReadPacketState::Error;
	}

	return EReadPacketState::Success;
}

bool FReplayHelper::ShouldSaveCheckpoint() const
{
	const double CHECKPOINT_DELAY = CVarCheckpointUploadDelayInSeconds.GetValueOnAnyThread();

	return (bPendingCheckpointRequest || ((DemoCurrentTime - LastCheckpointTime) > CHECKPOINT_DELAY));
}

float FReplayHelper::GetCheckpointSaveMaxMSPerFrame() const
{
	const float CVarValue = CVarCheckpointSaveMaxMSPerFrameOverride.GetValueOnAnyThread();
	if (CVarValue >= 0.0f)
	{
		return CVarValue;
	}

	return CheckpointSaveMaxMSPerFrame;
}

void FReplayHelper::ResetState()
{
	DemoFrameNum = 0;
	LatestReadFrameTime = 0.0f;
	bIsLoadingCheckpoint = false;
	LastCheckpointTime = 0.0f;
	ExternalDataToObjectMap.Empty();
	PlaybackFrames.Empty();

	ClearLevelStreamingState();
}

void FReplayHelper::OnLevelAddedToWorld(ULevel* InLevel, UWorld* InWorld)
{
	LLM_SCOPE(ELLMTag::Replays);

	if (InLevel && !InLevel->bClientOnlyVisible && (World == InWorld) && InWorld->IsPlayingReplay())
	{
		if (HasLevelStreamingFixes() && !NewStreamingLevelsThisFrame.Contains(InLevel) && !LevelsPendingFastForward.Contains(InLevel))
		{
			FLevelStatus& LevelStatus = FindOrAddLevelStatus(*InLevel);

			// If we haven't processed any packets for this level yet, immediately mark it as ready.
			if (!LevelStatus.bHasBeenSeen)
			{
				LevelStatus.bIsReady = true;
			}

			// If the level isn't ready, go ahead and queue it up to get fast-forwarded.
			// Note, we explicitly check the visible flag in case same the level gets notified multiple times.
			else if (!LevelStatus.bIsReady)
			{
				NewStreamingLevelsThisFrame.Add(InLevel);
			}
		}

		WeakLevelsByName.Add(InLevel->GetOutermost()->GetFName(), InLevel);
	}
}

void FReplayHelper::OnLevelRemovedFromWorld(ULevel* InLevel, UWorld* InWorld)
{
	if (InLevel && !InLevel->bClientOnlyVisible && (World == InWorld) && InWorld->IsPlayingReplay())
	{
		if (HasLevelStreamingFixes())
		{
			const FString LevelPackageName = GetLevelPackageName(*InLevel);
			if (LevelStatusesByName.Contains(LevelPackageName))
			{
				FLevelStatus& LevelStatus = GetLevelStatus(LevelPackageName);
				LevelStatus.bIsReady = false;

				// Make sure we don't try to fast-forward this level later.
				LevelsPendingFastForward.Remove(InLevel);
				NewStreamingLevelsThisFrame.Remove(InLevel);
			}
		}

		WeakLevelsByName.Remove(InLevel->GetOutermost()->GetFName());
	}

	// always invalidate cache since it uses pointers
	LevelStatusIndexByLevel.Remove(InLevel);
}

void FReplayHelper::AddOrUpdateEvent(const FString& Name, const FString& Group, const FString& Meta, const TArray<uint8>& Data)
{
	const uint32 SavedTimeMS = GetDemoCurrentTimeInMS();

	if (ReplayStreamer.IsValid())
	{
		ReplayStreamer->AddOrUpdateEvent(Name, SavedTimeMS, Group, Meta, Data);
	}

	UE_LOGF(LogDemo, Verbose, "AddOrUpdateEvent %ls.%ls. Total: %i, Time: %2.2f", *Group, *Name, Data.Num(), float(SavedTimeMS));
}

void FReplayHelper::ReadDeletedStartupActors(UNetConnection* Connection, FArchive& Ar, TSet<FString>& DeletedStartupActors)
{
	TArray<FString> TempList;
	Ar << TempList;

	DeletedStartupActors.Reserve(TempList.Num());

	for (FString& Path : TempList)
	{
		GEngine->NetworkRemapPath(Connection, Path, true);

		DeletedStartupActors.Emplace(MoveTemp(Path));
	}
}

void FReplayHelper::SetAnalyticsProvider(TSharedPtr<IAnalyticsProvider> InProvider)
{
	AnalyticsProvider = InProvider;

	if (ReplayStreamer.IsValid())
	{
		ReplayStreamer->SetAnalyticsProvider(InProvider);
	}
}

const TCHAR* LexToString(EReplayHeaderFlags Flag)
{
	switch (Flag)
	{
	case EReplayHeaderFlags::ClientRecorded:
		return TEXT("ClientRecorded");
	case EReplayHeaderFlags::HasStreamingFixes:
		return TEXT("HasStreamingFixes");
	case EReplayHeaderFlags::DeltaCheckpoints:
		return TEXT("DeltaCheckpoints");
	case EReplayHeaderFlags::GameSpecificFrameData:
		return TEXT("GameSpecificFrameData");
	case EReplayHeaderFlags::ReplayConnection:
		return TEXT("ReplayConnection");
	case EReplayHeaderFlags::ActorPrioritizationEnabled:
		return TEXT("ActorPrioritizationEnabled");
	case EReplayHeaderFlags::NetRelevancyEnabled:
		return TEXT("NetRelevancyEnabled");
	case EReplayHeaderFlags::AsyncRecorded:
		return TEXT("AsyncRecorded");
	default:
		check(false);
		return TEXT("Unknown");
	}
}

void FReplayHelper::RequestCheckpoint()
{
	bPendingCheckpointRequest = true;
}

void FReplayHelper::RemoveActorFromCheckpoint(UNetConnection* Connection, AActor* Actor)
{
	check(Connection);

	// if we're recording a checkpoint, and we have not yet passed the actor recording phase
	if (CheckpointSaveContext.CheckpointSaveState == ECheckpointSaveState::ProcessCheckpointActors)
	{
		// if there's already a channel open for this actor
		if (UActorChannel* Channel = Connection->FindActorChannelRef(Actor))
		{
			if (Channel->ChIndex != INDEX_NONE)
			{
				// if this actor is in the pending checkpoint actor list and we have not already recorded it
				if (int32* PendingIndex = CheckpointSaveContext.PendingActorToIndex.Find(Actor))
				{
					if (CheckpointSaveContext.PendingCheckpointActors.IsValidIndex(*PendingIndex)
						&& (*PendingIndex >= CheckpointSaveContext.NextAmortizedItem)
						&& (CheckpointSaveContext.PendingCheckpointActors[*PendingIndex].Actor.Get() == Actor))
					{
						UE_LOGF(LogDemo, Verbose, "Removing actor while it is still in the PendingCheckpointActors list: %ls", *GetNameSafe(Actor));

						FRepActorsCheckpointParams Params
						{
							FPlatformTime::Seconds(),
							(double)GetCheckpointSaveMaxMSPerFrame() / 1000
						};

						// force record it to the checkpoint now
						TArrayView<FPendingCheckPointActor> PendingView(CheckpointSaveContext.PendingCheckpointActors);
						int32 ActorIndex = 0;

						// Serialize the actor one last time before it gets removed
						ProcessCheckpointActors(Connection, PendingView.Slice(*PendingIndex, 1), ActorIndex, Params);

						// Clear the pending actor to avoid any further processing
						CheckpointSaveContext.PendingCheckpointActors[*PendingIndex].Actor = nullptr;

						// don't allow the channel index to be reused until we're done with the checkpoint
						Connection->AddReservedChannel(Channel->ChIndex);
					}
				}
			}
		}
	}
}

void FReplayHelper::NotifyActorDestroyed(UNetConnection* Connection, AActor* Actor)
{
	check(Actor);
	check(Connection);

	const bool bNetStartup = Actor->IsNetStartupActor();
	const bool bActorRewindable = Actor->bReplayRewindable;
	const bool bDeltaCheckpoint = HasDeltaCheckpoints();

	if (bNetStartup)
	{
		const FString FullName = Actor->GetFullName();

		// This was deleted due to a game interaction, which isn't supported for Rewindable actors (while recording).
		// However, since the actor is going to be deleted imminently, we need to track it.
		UE_CLOGF(bActorRewindable, LogDemo, Warning, "Replay Rewindable Actor destroyed during recording. Replay may show artifacts (%ls)", *FullName);

		UE_LOGF(LogDemo, VeryVerbose, "NotifyActorDestroyed: adding actor to deleted startup list: %ls", *FullName);
		RecordingDeletedNetStartupActors.Add(FullName);

		if (bDeltaCheckpoint)
		{
			RecordingDeltaCheckpointData.RecordingDeletedNetStartupActors.Add(FullName);
		}
	}
	else
	{
		if (bDeltaCheckpoint)
		{
			FNetworkGUID NetGUID = Connection->Driver->GetNetGuidCache()->NetGUIDLookup.FindRef(Actor);
			if (NetGUID.IsValid())
			{
				RecordingDeltaCheckpointData.DestroyedDynamicActors.Add(NetGUID);
			}
		}
	}
}

void FReplayHelper::NotifyReplayError(UE::Net::TNetResult<EReplayResult>&& Result)
{
	ResultManager.HandleNetResult(MoveTemp(Result));
}

void FReplayHelper::ResetDeltaCheckpointTracking(UNetConnection* Connection)
{
	DECLARE_SCOPE_CYCLE_COUNTER(TEXT("ReplayResetDeltaCheckpoint time"), STAT_ReplayResetDeltaCheckpoint, STATGROUP_Net);

	if (Connection)
	{
		if (UNetDriver* Driver = Connection->GetDriver())
		{
			// reset object list
			FNetworkObjectList& NetworkObjects = Driver->GetNetworkObjectList();
			NetworkObjects.ResetReplayDirtyTracking();

			// reset guid cache
			if (FNetGUIDCache* GuidCache = Connection->Driver->GetNetGuidCache().Get())
			{
				GuidCache->ResetReplayDirtyTracking();
			}

			// reset object replicators
			for (FObjectReplicator* Replicator : Driver->AllOwnedReplicators)
			{
				if (Replicator)
				{
					Replicator->ResetReplayDirtyTracking();
				}
			}
		}
	}
}

void FReplayResultHandler::InitResultHandler(FReplayHelper* InReplayHelper)
{
	ReplayHelper = InReplayHelper;
}

UE::Net::EHandleNetResult FReplayResultHandler::HandleNetResult(UE::Net::FNetResult&& InResult)
{
	using namespace UE::Net;

	UE_LOGF(LogDemo, Error, "FReplayResultHandler::HandleNetResult:");

	for (FNetResult::FConstIterator It(InResult); It; ++It)
	{
		UE_LOGF(LogDemo, Error, " - %ls", ToCStr(It->DynamicToString()));
	}

	TNetResult<EReplayResult>* CastedResult = Cast<EReplayResult>(&InResult);

	if (CastedResult && ReplayHelper)
	{
		if (ReplayHelper->bRecording)
		{
			ReplayHelper->OnReplayRecordError.ExecuteIfBound(*CastedResult);
		}
		else
		{
			ReplayHelper->OnReplayPlaybackError.ExecuteIfBound(*CastedResult);
		}
	}

	return EHandleNetResult::Handled;
}
