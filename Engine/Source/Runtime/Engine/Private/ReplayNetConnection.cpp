// Copyright Epic Games, Inc. All Rights Reserved.

#include "ReplayNetConnection.h"
#include "Engine/Level.h"
#include "Net/NetworkProfiler.h"
#include "Net/NetworkGranularMemoryLogging.h"
#include "Engine/LevelStreaming.h"
#include "Engine/Engine.h"
#include "Engine/World.h"
#include "Engine/ActorChannel.h"
#include "Engine/NetworkObjectList.h"
#include "NetworkReplayStreaming.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ReplayNetConnection)

static const int32 MAX_REPLAY_PACKET = 1024 * 2;

UReplayNetConnection::UReplayNetConnection(const FObjectInitializer& ObjectInitializer) : Super(ObjectInitializer)
{
	MaxPacket = MAX_REPLAY_PACKET;
	SetInternalAck(true);
	SetReplay(true);
	SetAutoFlush(true);
	SetUnlimitedBunchSizeAllowed(true);
}

void UReplayNetConnection::InitConnection(UNetDriver* InDriver, EConnectionState InState, const FURL& InURL, int32 InConnectionSpeed, int32 InMaxPacket)
{
	// default implementation
	Super::InitConnection(InDriver, InState, InURL, InConnectionSpeed);

	MaxPacket = (InMaxPacket == 0 || InMaxPacket > MAX_REPLAY_PACKET) ? MAX_REPLAY_PACKET : InMaxPacket;

	SetInternalAck(true);
	SetReplay(true);
	SetAutoFlush(true);
	SetUnlimitedBunchSizeAllowed(true);

	InitSendBuffer();

	ReplayHelper.Init(InURL);
	ReplayHelper.bRecording = true;
}

void UReplayNetConnection::CleanUp()
{
	Super::CleanUp();

	ReplayHelper.StopReplay();

	FWorldDelegates::LevelRemovedFromWorld.Remove(OnLevelRemovedFromWorldHandle);
	FWorldDelegates::LevelAddedToWorld.Remove(OnLevelAddedToWorldHandle);
}

void UReplayNetConnection::StartRecording()
{
	if (UWorld* World = GetWorld())
	{
		for (ULevelStreaming* LevelStreaming : World->GetStreamingLevels())
		{
			if (LevelStreaming)
			{
				const ULevel* Level = LevelStreaming->GetLoadedLevel();
				if (Level && Level->bIsVisible && !Level->bClientOnlyVisible)
				{
					FUpdateLevelVisibilityLevelInfo LevelVisibility(Level, true);
					LevelVisibility.PackageName = NetworkRemapPath(LevelVisibility.PackageName, true);

					UpdateLevelVisibility(LevelVisibility);
				}
			}
		}
	}

	OnLevelRemovedFromWorldHandle = FWorldDelegates::LevelRemovedFromWorld.AddUObject(this, &ThisClass::OnLevelRemovedFromWorld);
	OnLevelAddedToWorldHandle = FWorldDelegates::LevelAddedToWorld.AddUObject(this, &ThisClass::OnLevelAddedToWorld);

	ReplayHelper.StartRecording(this);
	ReplayHelper.CreateSpectatorController(this);
}

void UReplayNetConnection::Tick(float DeltaSeconds)
{
	Super::Tick(DeltaSeconds);

	DemoFrameNum++;

	ReplayHelper.TickRecording(DeltaSeconds, this);
}

void UReplayNetConnection::Serialize(FArchive& Ar)
{
	GRANULAR_NETWORK_MEMORY_TRACKING_INIT(Ar, "UReplayNetConnection::Serialize");

	GRANULAR_NETWORK_MEMORY_TRACKING_TRACK("Super", Super::Serialize(Ar));

	if (Ar.IsCountingMemory())
	{
		ReplayHelper.Serialize(Ar);
	}
}

FString UReplayNetConnection::LowLevelGetRemoteAddress(bool bAppendPort)
{
	return TEXT("UReplayNetConnection");
}

void UReplayNetConnection::LowLevelSend(void* Data, int32 CountBits, FOutPacketTraits& Traits)
{
	uint32 CountBytes = FMath::DivideAndRoundUp(CountBits, 8);

	if (CountBytes == 0)
	{
		UE_LOGF(LogDemo, Warning, "UReplayNetConnection::LowLevelSend: Ignoring empty packet.");
		return;
	}

	if (CountBytes > MAX_REPLAY_PACKET)
	{
		UE_LOGF(LogDemo, Fatal, "UReplayNetConnection::LowLevelSend: CountBytes > MAX_REPLAY_PACKET.");
	}

	TrackSendForProfiler(Data, CountBytes);

	const bool bCheckpoint = (ResendAllDataState != EResendAllDataState::None);

	TArray<FQueuedDemoPacket>& QueuedPackets = bCheckpoint ? ReplayHelper.QueuedCheckpointPackets : ReplayHelper.QueuedDemoPackets;

	int32 NewIndex = QueuedPackets.Emplace((uint8*)Data, CountBits, Traits);

	if (ULevel* Level = GetRepContextLevel())
	{
		QueuedPackets[NewIndex].SeenLevelIndex = ReplayHelper.FindOrAddLevelStatus(*Level).LevelIndex + 1;

		if (AActor* Actor = GetRepContextActor())
		{
			if (!bCheckpoint && ReplayHelper.bHasDeltaCheckpoints && Driver)
			{
				Driver->GetNetworkObjectList().MarkDirtyForReplay(Actor);
			}
		}
	}
	else
	{
		UE_LOGF(LogDemo, Warning, "UReplayNetConnection::LowLevelSend - Missing rep context.");
	}
}

void UReplayNetConnection::TrackSendForProfiler(const void* Data, int32 NumBytes)
{
	NETWORK_PROFILER(GNetworkProfiler.FlushOutgoingBunches(this));

	// Track "socket send" even though we're not technically sending to a socket, to get more accurate information in the profiler.
	NETWORK_PROFILER(GNetworkProfiler.TrackSocketSendToCore(TEXT("Unreal"), Data, NumBytes, NumPacketIdBits, NumBunchBits, NumAckBits, NumPaddingBits, this));
}

FString UReplayNetConnection::LowLevelDescribe()
{
	return TEXT("Replay recording connection");
}

int32 UReplayNetConnection::IsNetReady(bool Saturate)
{
	return 1;
}

bool UReplayNetConnection::IsReplayReady() const
{
	return (ResendAllDataState == EResendAllDataState::None);
}


FName UReplayNetConnection::NetworkRemapPath(FName InPackageName, bool bReading)
{
	// For PIE Networking: remap the packagename to our local PIE packagename
	FString PackageNameStr = InPackageName.ToString();
	GEngine->NetworkRemapPath(this, PackageNameStr, bReading);
	return FName(*PackageNameStr);
}

void UReplayNetConnection::HandleClientPlayer(APlayerController* PC, UNetConnection* NetConnection)
{
	// intentionally skipping Super
}

void UReplayNetConnection::OnLevelRemovedFromWorld(ULevel* Level, UWorld* World)
{
	if (GetWorld() == World)
	{
		if (Level && !Level->bClientOnlyVisible)
		{
			FUpdateLevelVisibilityLevelInfo LevelVisibility(Level, false);
			LevelVisibility.PackageName = NetworkRemapPath(LevelVisibility.PackageName, false);

			UpdateLevelVisibility(LevelVisibility);
		}
	}
}

void UReplayNetConnection::OnLevelAddedToWorld(ULevel* Level, UWorld* World)
{
	if (GetWorld() == World)
	{
		if (Level && !Level->bClientOnlyVisible)
		{
			FUpdateLevelVisibilityLevelInfo LevelVisibility(Level, true);
			LevelVisibility.PackageName = NetworkRemapPath(LevelVisibility.PackageName, false);

			UpdateLevelVisibility(LevelVisibility);
		}
	}
}

TSharedPtr<const FInternetAddr> UReplayNetConnection::GetRemoteAddr()
{
	return FInternetAddrDemo::DemoInternetAddr;
}

bool UReplayNetConnection::ClientHasInitializedLevel(const ULevel* TestLevel) const
{
	return (DemoFrameNum > 2 || Super::ClientHasInitializedLevel(TestLevel));
}

void UReplayNetConnection::AddEvent(const FString& Group, const FString& Meta, const TArray<uint8>& Data)
{
	AddOrUpdateEvent(FString(), Group, Meta, Data);
}

void UReplayNetConnection::AddOrUpdateEvent(const FString& EventName, const FString& Group, const FString& Meta, const TArray<uint8>& Data)
{
	ReplayHelper.AddOrUpdateEvent(EventName, Group, Meta, Data);
}

bool UReplayNetConnection::IsSavingCheckpoint() const
{
	return (ResendAllDataState != EResendAllDataState::None);
}

void UReplayNetConnection::AddUserToReplay(const FString& UserString)
{
	if (ReplayHelper.ReplayStreamer.IsValid())
	{
		ReplayHelper.ReplayStreamer->AddUserToReplay(UserString);
	}
}

void UReplayNetConnection::OnSeamlessTravelStart(UWorld* CurrentWorld, const FString& LevelName)
{
	ReplayHelper.OnSeamlessTravelStart(CurrentWorld, LevelName, this);
}

void UReplayNetConnection::NotifyActorDestroyed(AActor* Actor, bool IsSeamlessTravel /* = false */)
{
	if (!IsSeamlessTravel)
	{
		ReplayHelper.NotifyActorDestroyed(this, Actor);
	}

	Super::NotifyActorDestroyed(Actor, IsSeamlessTravel);
}

void UReplayNetConnection::SetAnalyticsProvider(TSharedPtr<IAnalyticsProvider> InProvider)
{
	ReplayHelper.SetAnalyticsProvider(InProvider);
}

void UReplayNetConnection::SetCheckpointSaveMaxMSPerFrame(const float InCheckpointSaveMaxMSPerFrame)
{
	ReplayHelper.CheckpointSaveMaxMSPerFrame = InCheckpointSaveMaxMSPerFrame;
}

void UReplayNetConnection::NotifyActorChannelCleanedUp(UActorChannel* Channel, EChannelCloseReason CloseReason)
{
	Super::NotifyActorChannelCleanedUp(Channel, CloseReason);

	if (ReplayHelper.HasDeltaCheckpoints() && (ReplayHelper.GetCheckpointSaveState() == FReplayHelper::ECheckpointSaveState::Idle))
	{
		if (Channel && Channel->bOpenedForCheckpoint)
		{
			ReplayHelper.RecordingDeltaCheckpointData.ChannelsToClose.Add(Channel->ActorNetGUID, CloseReason);
		}
	}
}

void UReplayNetConnection::RequestCheckpoint()
{
	ReplayHelper.RequestCheckpoint();
}

bool UReplayNetConnection::SetExternalDataForObject(UObject* OwningObject, const uint8* Src, const int32 NumBits)
{
	return ReplayHelper.SetExternalDataForObject(this, OwningObject, Src, NumBits);
}
