// Copyright Epic Games, Inc. All Rights Reserved.

#include "ReplaySubsystem.h"
#include "Engine/Engine.h"
#include "Engine/DemoNetDriver.h"
#include "Engine/GameInstance.h"
#include "Engine/NetworkObjectList.h"
#include "Engine/World.h"
#include "Misc/CommandLine.h"
#include "ReplayNetConnection.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ReplaySubsystem)

namespace ReplaySubsystem
{
	TAutoConsoleVariable<bool> CVarUseReplayConnection(TEXT("Replay.UseReplayConnection"), false, TEXT(""));
};

void UReplaySubsystem::Initialize(FSubsystemCollectionBase& Collection)
{
	Super::Initialize(Collection);

	FWorldDelegates::OnSeamlessTravelStart.AddUObject(this, &UReplaySubsystem::OnSeamlessTravelStart);
	FWorldDelegates::OnSeamlessTravelTransition.AddUObject(this, &UReplaySubsystem::OnSeamlessLevelTransition);
	FWorldDelegates::OnCopyWorldData.AddUObject(this, &UReplaySubsystem::OnCopyWorldData);
}

void UReplaySubsystem::Deinitialize()
{
	FWorldDelegates::OnCopyWorldData.RemoveAll(this);
	FWorldDelegates::OnSeamlessTravelTransition.RemoveAll(this);

	Super::Deinitialize();
}

// If we are doing seamless travel for replay playback, then make sure to transfer the replay driver over to the new world
void UReplaySubsystem::OnCopyWorldData(UWorld* CurrentWorld, UWorld* LoadedWorld)
{
	if (CurrentWorld == GetWorld())
	{
		UDemoNetDriver* DemoNetDriver = CurrentWorld->GetDemoNetDriver();

		FLevelCollection* const CurrentCollection = CurrentWorld->FindCollectionByType(ELevelCollectionType::DynamicSourceLevels);
		FLevelCollection* const LoadedCollection = LoadedWorld->FindCollectionByType(ELevelCollectionType::DynamicSourceLevels);

		if (DemoNetDriver && (DemoNetDriver->IsPlaying() || DemoNetDriver->IsRecordingMapChanges()))
		{
			CurrentWorld->SetDemoNetDriver(nullptr);
			DemoNetDriver->SetWorld(LoadedWorld);
			LoadedWorld->SetDemoNetDriver(DemoNetDriver);

			if (CurrentCollection && LoadedCollection)
			{
				LoadedCollection->SetDemoNetDriver(DemoNetDriver);
				CurrentCollection->SetDemoNetDriver(nullptr);
			}
		}
		else
		{
			StopExistingReplays(CurrentWorld);

			if (CurrentCollection)
			{
				CurrentCollection->SetNetDriver(nullptr);
			}
		}
	}
}

void UReplaySubsystem::OnSeamlessTravelStart(UWorld* CurrentWorld, const FString& LevelName)
{
	if (CurrentWorld == GetWorld())
	{
		if (UDemoNetDriver* DemoNetDriver = CurrentWorld->GetDemoNetDriver())
		{
			DemoNetDriver->OnSeamlessTravelStartDuringRecording(LevelName);
		}

		if (UReplayNetConnection* Connection = ReplayConnection.Get())
		{
			Connection->OnSeamlessTravelStart(CurrentWorld, LevelName);
		}
	}
}

void UReplaySubsystem::OnSeamlessLevelTransition(UWorld* CurrentWorld)
{
	if (CurrentWorld == GetWorld())
	{
		// If it's not still playing, destroy the demo net driver before we start renaming actors.
		if (UDemoNetDriver* DemoNetDriver = CurrentWorld->GetDemoNetDriver())
		{
			if (!DemoNetDriver->IsPlaying() && !DemoNetDriver->IsRecordingMapChanges())
			{
				StopExistingReplays(CurrentWorld);
			}
		}
		else
		{
			StopExistingReplays(CurrentWorld);
		}
	}
}

void UReplaySubsystem::RecordReplay(const FString& Name, const FString& FriendlyName, const TArray<FString>& AdditionalOptions, TSharedPtr<IAnalyticsProvider> AnalyticsProvider)
{
	LLM_SCOPE(ELLMTag::Replays);

	if (FParse::Param(FCommandLine::Get(), TEXT("NOREPLAYS")))
	{
		UE_LOGF(LogDemo, Warning, "UReplaySubsystem::RecordReplay: Rejected due to -noreplays option");
		return;
	}

	UWorld* CurrentWorld = GetWorld();

	if (CurrentWorld == nullptr)
	{
		UE_LOGF(LogDemo, Warning, "UReplaySubsystem::RecordReplay: GetWorld() is null");
		return;
	}

	if (CurrentWorld->IsPlayingReplay())
	{
		UE_LOGF(LogDemo, Warning, "UReplaySubsystem::RecordReplay: A replay is already playing, cannot begin recording another one.");
		return;
	}

	FURL DemoURL;
	FString DemoName = Name;

	DemoName.ReplaceInline(TEXT("%m"), *CurrentWorld->GetMapName());

	// replace the current URL's map with a demo extension
	DemoURL.Map = DemoName;
	DemoURL.AddOption(*FString::Printf(TEXT("DemoFriendlyName=%s"), *FriendlyName));

	for (const FString& Option : AdditionalOptions)
	{
		DemoURL.AddOption(*Option);
	}

	const UE::ReplaySubsystem::EStopReplayFlags StopExistingFlags = DemoURL.HasOption(TEXT("flush")) ? UE::ReplaySubsystem::EStopReplayFlags::Flush : UE::ReplaySubsystem::EStopReplayFlags::None;

	UNetDriver* NetDriver = CurrentWorld->GetNetDriver();

	// must be server and using a replication graph to use a replay connection
	if (NetDriver && NetDriver->IsServer() && NetDriver->GetReplicationDriver() && ReplaySubsystem::CVarUseReplayConnection.GetValueOnAnyThread())
	{
		StopExistingReplays(CurrentWorld, StopExistingFlags);

		UReplayNetConnection* Connection = NewObject<UReplayNetConnection>();

		ReplayConnection = Connection;

		Connection->InitConnection(NetDriver, USOCK_Open, DemoURL, 1000000);
		Connection->SetAnalyticsProvider(AnalyticsProvider);

		NetDriver->AddClientConnection(Connection);
		
		UE_LOGF(LogDemo, Log, "UReplaySubsystem::RecordReplay: Starting recording with replay connection.  Name: %ls FriendlyName: %ls", *Name, *FriendlyName);

		Connection->StartRecording();

		return;
	}

	bool bDestroyedDemoNetDriver = false;

	UDemoNetDriver* DemoNetDriver = CurrentWorld->GetDemoNetDriver();

	if (!DemoNetDriver || !DemoNetDriver->IsRecordingMapChanges() || !DemoNetDriver->IsRecordingPaused())
	{
		StopExistingReplays(CurrentWorld, StopExistingFlags);

		bDestroyedDemoNetDriver = true;

		if (!GEngine->CreateNamedNetDriver(CurrentWorld, NAME_DemoNetDriver, NAME_DemoNetDriver))
		{
			UE_LOGF(LogDemo, Warning, "RecordReplay: failed to create demo net driver!");
			return;
		}

		DemoNetDriver = Cast<UDemoNetDriver>(GEngine->FindNamedNetDriver(CurrentWorld, NAME_DemoNetDriver));

		CurrentWorld->SetDemoNetDriver(DemoNetDriver);
	}

	check(DemoNetDriver != nullptr);

	DemoNetDriver->SetAnalyticsProvider(AnalyticsProvider);
	DemoNetDriver->SetWorld(CurrentWorld);

	// Set the new demo driver as the current collection's driver
	FLevelCollection* CurrentLevelCollection = CurrentWorld->FindCollectionByType(ELevelCollectionType::DynamicSourceLevels);
	if (CurrentLevelCollection)
	{
		CurrentLevelCollection->SetDemoNetDriver(DemoNetDriver);
	}

	UE_LOGF(LogDemo, Log, "UReplaySubsystem::RecordReplay: Starting recording with demo driver.  Name: %ls FriendlyName: %ls", *Name, *FriendlyName);

	FString Error;

	if (bDestroyedDemoNetDriver)
	{
		if (!DemoNetDriver->InitListen(CurrentWorld, DemoURL, false, Error))
		{
			UE_LOGF(LogDemo, Warning, "Demo recording - InitListen failed: %ls", *Error);
			CurrentWorld->SetDemoNetDriver(nullptr);
			return;
		}
	}
	else if (!DemoNetDriver->ContinueListen(DemoURL))
	{
		UE_LOGF(LogDemo, Warning, "Demo recording - ContinueListen failed");
		CurrentWorld->SetDemoNetDriver(nullptr);
		return;
	}

	UE_LOGF(LogDemo, Verbose, "Num Network Actors: %i", DemoNetDriver->GetNetworkObjectList().GetActiveObjects().Num());
}

bool UReplaySubsystem::PlayReplay(const FString& Name, UWorld* WorldOverride, const TArray<FString>& AdditionalOptions)
{
	LLM_SCOPE(ELLMTag::Replays);

	UWorld* CurrentWorld = WorldOverride != nullptr ? WorldOverride : GetWorld();

	if (CurrentWorld == nullptr)
	{
		UE_LOGF(LogDemo, Warning, "UReplaySubsystem::PlayReplay: GetWorld() is null");
		return false;
	}

	FURL DemoURL;
	DemoURL.Map = Name;

	for (const FString& Option : AdditionalOptions)
	{
		DemoURL.AddOption(*Option);
	}

	StopExistingReplays(CurrentWorld, DemoURL.HasOption(TEXT("flush")) ? UE::ReplaySubsystem::EStopReplayFlags::Flush : UE::ReplaySubsystem::EStopReplayFlags::None);

	UE_LOGF(LogDemo, Log, "PlayReplay: Attempting to play demo %ls", *Name);

	if (!GEngine->CreateNamedNetDriver(CurrentWorld, NAME_DemoNetDriver, NAME_DemoNetDriver))
	{
		UE_LOGF(LogDemo, Warning, "PlayReplay: failed to create demo net driver!");
		return false;
	}

	CurrentWorld->SetDemoNetDriver(Cast<UDemoNetDriver>(GEngine->FindNamedNetDriver(CurrentWorld, NAME_DemoNetDriver)));

	UDemoNetDriver* DemoNetDriver = CurrentWorld->GetDemoNetDriver();

	check(DemoNetDriver != nullptr);

	DemoNetDriver->SetWorld(CurrentWorld);

	FString Error;

	if (!DemoNetDriver->InitConnect(CurrentWorld, DemoURL, Error))
	{
		UE_LOGF(LogDemo, Warning, "Demo playback failed: %ls", *Error);
		CurrentWorld->DestroyDemoNetDriver();
		return false;
	}
	else
	{
		FCoreUObjectDelegates::PostDemoPlay.Broadcast();
	}

	return true;
}

void UReplaySubsystem::StopReplay()
{
	if (UWorld* CurrentWorld = GetWorld())
	{
		const bool bWasReplaying = CurrentWorld->IsPlayingReplay();

		StopExistingReplays(CurrentWorld);

		if (UGameInstance* GameInstance = GetGameInstance())
		{
			if (bWasReplaying && bLoadDefaultMapOnStop)
			{
				GEngine->BrowseToDefaultMap(*GameInstance->GetWorldContext());
			}
		}
	}
	else
	{
		UE_LOGF(LogDemo, Warning, "UReplaySubsystem::StopRecordingReplay: GetWorld() is null");
	}
}

void UReplaySubsystem::StopExistingReplays(UWorld* InWorld, UE::ReplaySubsystem::EStopReplayFlags Flags)
{
	UWorld* CurrentWorld = InWorld ? InWorld : GetWorld();

	if (CurrentWorld)
	{
		CurrentWorld->DestroyDemoNetDriver();
	}

	if (UReplayNetConnection* Connection = ReplayConnection.Get())
	{
		Connection->CleanUp();
		ReplayConnection = nullptr;
	}

	if (EnumHasAnyFlags(Flags, UE::ReplaySubsystem::EStopReplayFlags::Flush))
	{
		//@todo: narrow to specific streamer that was stopped
		FNetworkReplayStreaming::Get().Flush();
	}
}

FString UReplaySubsystem::GetActiveReplayName() const
{
	UWorld* CurrentWorld = GetWorld();

	if (CurrentWorld != nullptr && CurrentWorld->GetDemoNetDriver() != nullptr)
	{
		return CurrentWorld->GetDemoNetDriver()->GetActiveReplayName();
	}

	if (UReplayNetConnection* Connection = ReplayConnection.Get())
	{
		return Connection->GetActiveReplayName();
	}

	return FString();
}

float UReplaySubsystem::GetReplayTotalTime() const
{
	UWorld* CurrentWorld = GetWorld();

	if (CurrentWorld != nullptr && CurrentWorld->GetDemoNetDriver() != nullptr)
	{
		return CurrentWorld->GetDemoNetDriver()->GetDemoTotalTime();
	}

	if (UReplayNetConnection* Connection = ReplayConnection.Get())
	{
		return Connection->GetReplayTotalTime();
	}
	
	return 0.0f;
}

float UReplaySubsystem::GetReplayCurrentTime() const
{
	UWorld* CurrentWorld = GetWorld();

	if (CurrentWorld != nullptr && CurrentWorld->GetDemoNetDriver() != nullptr)
	{
		return CurrentWorld->GetDemoNetDriver()->GetDemoCurrentTime();
	}

	if (UReplayNetConnection* Connection = ReplayConnection.Get())
	{
		return Connection->GetReplayCurrentTime();
	}

	return 0.0f;
}

void UReplaySubsystem::AddUserToReplay(const FString& UserString)
{
	UWorld* CurrentWorld = GetWorld();

	if (CurrentWorld != nullptr && CurrentWorld->GetDemoNetDriver() != nullptr)
	{
		CurrentWorld->GetDemoNetDriver()->AddUserToReplay(UserString);
	}

	if (UReplayNetConnection* Connection = ReplayConnection.Get())
	{
		Connection->AddUserToReplay(UserString);
	}
}

bool UReplaySubsystem::IsRecording() const
{
	UWorld* CurrentWorld = GetWorld();

	if (CurrentWorld != nullptr && CurrentWorld->GetDemoNetDriver() != nullptr)
	{
		return CurrentWorld->GetDemoNetDriver()->IsRecording();
	}

	return (ReplayConnection.Get() != nullptr);
}

bool UReplaySubsystem::IsPlaying() const
{
	UWorld* CurrentWorld = GetWorld();

	if (CurrentWorld != nullptr && CurrentWorld->GetDemoNetDriver() != nullptr)
	{
		return CurrentWorld->GetDemoNetDriver()->IsPlaying();
	}

	return false;
}

void UReplaySubsystem::AddEvent(const FString& Group, const FString& Meta, const TArray<uint8>& Data)
{
	LLM_SCOPE(ELLMTag::Replays);

	UWorld* CurrentWorld = GetWorld();

	if (CurrentWorld != nullptr && CurrentWorld->GetDemoNetDriver() != nullptr)
	{
		CurrentWorld->GetDemoNetDriver()->AddEvent(Group, Meta, Data);
	}

	if (UReplayNetConnection* Connection = ReplayConnection.Get())
	{
		Connection->AddEvent(Group, Meta, Data);
	}
}

void UReplaySubsystem::AddOrUpdateEvent(const FString& EventName, const FString& Group, const FString& Meta, const TArray<uint8>& Data)
{
	LLM_SCOPE(ELLMTag::Replays);

	UWorld* CurrentWorld = GetWorld();

	if (CurrentWorld != nullptr && CurrentWorld->GetDemoNetDriver() != nullptr)
	{
		CurrentWorld->GetDemoNetDriver()->AddOrUpdateEvent(EventName, Group, Meta, Data);
	}

	if (UReplayNetConnection* Connection = ReplayConnection.Get())
	{
		Connection->AddOrUpdateEvent(EventName, Group, Meta, Data);
	}
}

bool UReplaySubsystem::IsSavingCheckpoint() const
{
	UWorld* CurrentWorld = GetWorld();

	if (CurrentWorld != nullptr && CurrentWorld->GetDemoNetDriver() != nullptr)
	{
		return CurrentWorld->GetDemoNetDriver()->IsSavingCheckpoint();
	}

	if (UReplayNetConnection* Connection = ReplayConnection.Get())
	{
		return Connection->IsSavingCheckpoint();
	}

	return false;
}

bool UReplaySubsystem::IsReplayDeferred(const FString& ReplayName) const
{
	// Default implementation of replay subsystem does not defer replays 
	return false;
}

void UReplaySubsystem::SetCheckpointSaveMaxMSPerFrame(const float InCheckpointSaveMaxMSPerFrame)
{
	UWorld* CurrentWorld = GetWorld();

	if (CurrentWorld != nullptr && CurrentWorld->GetDemoNetDriver() != nullptr)
	{
		return CurrentWorld->GetDemoNetDriver()->SetCheckpointSaveMaxMSPerFrame(InCheckpointSaveMaxMSPerFrame);
	}

	if (UReplayNetConnection* Connection = ReplayConnection.Get())
	{
		return Connection->SetCheckpointSaveMaxMSPerFrame(InCheckpointSaveMaxMSPerFrame);
	}
}

void UReplaySubsystem::RequestCheckpoint()
{
	UWorld* CurrentWorld = GetWorld();

	if (CurrentWorld != nullptr && CurrentWorld->GetDemoNetDriver() != nullptr)
	{
		return CurrentWorld->GetDemoNetDriver()->RequestCheckpoint();
	}

	if (UReplayNetConnection* Connection = ReplayConnection.Get())
	{
		return Connection->RequestCheckpoint();
	}
}

void UReplaySubsystem::SetExternalDataForObject(UObject* OwningObject, const uint8* Src, const int32 NumBits)
{
	LLM_SCOPE(ELLMTag::Replays);

	UWorld* CurrentWorld = GetWorld();

	if (CurrentWorld != nullptr && CurrentWorld->GetDemoNetDriver() != nullptr)
	{
		CurrentWorld->GetDemoNetDriver()->SetExternalDataForObject(OwningObject, Src, NumBits);
	}

	if (UReplayNetConnection* Connection = ReplayConnection.Get())
	{
		Connection->SetExternalDataForObject(OwningObject, Src, NumBits);
	}
}
