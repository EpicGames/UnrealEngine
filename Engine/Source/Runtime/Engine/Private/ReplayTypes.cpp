// Copyright Epic Games, Inc. All Rights Reserved.

#include "ReplayTypes.h"
#include "Misc/NetworkVersion.h"
#include "Net/NetworkGranularMemoryLogging.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ReplayTypes)

const FGuid FReplayCustomVersion::Guid = FGuid(0x8417998A, 0xBBC043EC, 0x81B3D119, 0x072D2722);
FCustomVersionRegistration GRegisterReplayCustomVersion(FReplayCustomVersion::Guid, FReplayCustomVersion::LatestVersion, TEXT("Replay"));

void FDeltaCheckpointData::CountBytes(FArchive& Ar) const
{
	GRANULAR_NETWORK_MEMORY_TRACKING_INIT(Ar, "FDeltaCheckpointData::CountBytes");

	GRANULAR_NETWORK_MEMORY_TRACKING_TRACK("RecordingDeletedNetStartupActors", RecordingDeletedNetStartupActors.CountBytes(Ar));
	GRANULAR_NETWORK_MEMORY_TRACKING_TRACK("DestroyedNetStartupActors", DestroyedNetStartupActors.CountBytes(Ar));
	GRANULAR_NETWORK_MEMORY_TRACKING_TRACK("DestroyedDynamicActors", DestroyedDynamicActors.CountBytes(Ar));
	GRANULAR_NETWORK_MEMORY_TRACKING_TRACK("ChannelsToClose", ChannelsToClose.CountBytes(Ar));
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
FNetworkDemoHeader::FNetworkDemoHeader() :
	Magic(NETWORK_DEMO_MAGIC),
	Version((uint32)FReplayCustomVersion::CustomVersions),
	NetworkChecksum(0),
	EngineNetworkProtocolVersion(FEngineNetworkCustomVersion::LatestVersion),
	GameNetworkProtocolVersion(FGameNetworkCustomVersion::LatestVersion),
	Guid(),
	MinRecordHz(0.0f),
	MaxRecordHz(0.0f),
	FrameLimitInMS(0.0f),
	CheckpointLimitInMS(0.0f),
	BuildConfig(EBuildConfiguration::Unknown),
	BuildTarget(EBuildTargetType::Unknown),
	EngineVersion(FEngineVersion::Current()),
	HeaderFlags(EReplayHeaderFlags::None),
	PackageVersionUE(GPackageFileUEVersion),
	PackageVersionLicenseeUE(GPackageFileLicenseeUEVersion)
{
}

void FNetworkDemoHeader::SetDefaultNetworkVersions()
{
	NetworkChecksum = FNetworkVersion::GetLocalNetworkVersion();
	EngineNetworkProtocolVersion = FNetworkVersion::GetNetworkProtocolVersion(FEngineNetworkCustomVersion::Guid);
	GameNetworkProtocolVersion = FNetworkVersion::GetNetworkProtocolVersion(FGameNetworkCustomVersion::Guid);

	CustomVersions = FNetworkVersion::GetNetworkCustomVersions();
	CustomVersions.SetVersion(FReplayCustomVersion::Guid, FReplayCustomVersion::LatestVersion, TEXT("Replay"));
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

uint32 FNetworkDemoHeader::GetCustomVersion(const FGuid& VersionGuid) const
{
	const FCustomVersion* CustomVer = CustomVersions.GetVersion(VersionGuid);
	return CustomVer != nullptr ? CustomVer->Version : 0;
}

void FNetworkDemoHeader::CountBytes(FArchive& Ar) const 
{
	GRANULAR_NETWORK_MEMORY_TRACKING_INIT(Ar, "FNetworkDemoHeader::CountBytes");

	GRANULAR_NETWORK_MEMORY_TRACKING_TRACK("LevelNamesAndTimes",
		LevelNamesAndTimes.CountBytes(Ar);
		for (const FLevelNameAndTime& LevelNameAndTime : LevelNamesAndTimes)
		{
			LevelNameAndTime.CountBytes(Ar);
		}
	);

	GRANULAR_NETWORK_MEMORY_TRACKING_TRACK("GameSpecificData",
		GameSpecificData.CountBytes(Ar);
		for (const FString& Datum : GameSpecificData)
		{
			Datum.CountBytes(Ar);
		}
	);
}

FArchive& operator<<(FArchive& Ar, FNetworkDemoHeader& Header)
{
	Ar << Header.Magic;

	// Check magic value
	if (Header.Magic != NETWORK_DEMO_MAGIC)
	{
		UE_LOGF(LogDemo, Error, "Header.Magic != NETWORK_DEMO_MAGIC");
		Ar.SetError();
		return Ar;
	}

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	Ar << Header.Version;

	// Check legacy version
	if (Header.Version < (uint32)FReplayCustomVersion::MinSupportedVersion)
	{
		UE_LOGF(LogDemo, Error, "Header.Version < MinSupportedVersion. Header.Version: %i, MinSupportedVersion: %i", Header.Version, FReplayCustomVersion::MinSupportedVersion);
		Ar.SetError();
		return Ar;
	}

	FReplayCustomVersion::Type ReplayVersion = (FReplayCustomVersion::Type)Header.Version;

	if (Ar.IsLoading())
	{
		Header.CustomVersions.Empty();
	}

	if (ReplayVersion >= FReplayCustomVersion::CustomVersions)
	{
		checkf(!Ar.IsSaving() || Header.CustomVersions.GetVersion(FReplayCustomVersion::Guid) != nullptr, TEXT("A valid replay custom version should exist when saving."));

		Header.CustomVersions.Serialize(Ar);

		if (Ar.IsLoading())
		{
			Ar.SetCustomVersions(Header.CustomVersions);

			ReplayVersion = (FReplayCustomVersion::Type)Ar.CustomVer(FReplayCustomVersion::Guid);
		}
		else
		{
			Ar.UsingCustomVersion(FReplayCustomVersion::Guid);
			Ar.UsingCustomVersion(FEngineNetworkCustomVersion::Guid);
			Ar.UsingCustomVersion(FGameNetworkCustomVersion::Guid);
		}
	}

	Ar << Header.NetworkChecksum;
	Ar << Header.EngineNetworkProtocolVersion;
	Ar << Header.GameNetworkProtocolVersion;

	if (ReplayVersion < FReplayCustomVersion::CustomVersions)
	{
		if (Ar.IsLoading())
		{
			Ar.SetCustomVersion(FReplayCustomVersion::Guid, Header.Version, TEXT("Replay"));
			Ar.SetCustomVersion(FEngineNetworkCustomVersion::Guid, Header.EngineNetworkProtocolVersion, TEXT("EngineNetwork"));
			Ar.SetCustomVersion(FGameNetworkCustomVersion::Guid, Header.GameNetworkProtocolVersion, TEXT("GameNetwork"));

			Header.CustomVersions.SetVersion(FReplayCustomVersion::Guid, Header.Version, TEXT("Replay"));
			Header.CustomVersions.SetVersion(FEngineNetworkCustomVersion::Guid, Header.EngineNetworkProtocolVersion, TEXT("EngineNetwork"));
			Header.CustomVersions.SetVersion(FGameNetworkCustomVersion::Guid, Header.GameNetworkProtocolVersion, TEXT("GameNetwork"));
		}
	}
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	if (Ar.IsLoading())
	{
		TArray<FCustomVersionDifference> VersionDiffs = FCurrentCustomVersions::Compare(Header.CustomVersions.GetAllVersions(), TEXT("Replay"));
		for (const FCustomVersionDifference& Diff : VersionDiffs)
		{
			if (Diff.Type == ECustomVersionDifference::Missing)
			{
				UE_LOGF(LogDemo, Error, "Replay was saved with a custom version that is not present. Tag %ls Version %d", *Diff.Version->Key.ToString(), Diff.Version->Version);
				Ar.SetError();
				return Ar;
			}
			else if (Diff.Type == ECustomVersionDifference::Invalid)
			{
				UE_LOGF(LogDemo, Error, "Replay was saved with an invalid custom version. Tag %ls Version %d", *Diff.Version->Key.ToString(), Diff.Version->Version);
				Ar.SetError();
				return Ar;
			}
			else if (Diff.Type == ECustomVersionDifference::Newer)
			{
				FCustomVersion MaxExpectedVersion = FCurrentCustomVersions::Get(Diff.Version->Key).GetValue();

				UE_LOGF(LogDemo, Error, "Replay was saved with a newer custom version than the current. Tag %ls Name '%ls' ReplayVersion %d  MaxExpected %d",
					*Diff.Version->Key.ToString(), *MaxExpectedVersion.GetFriendlyName().ToString(), Diff.Version->Version, MaxExpectedVersion.Version);
				Ar.SetError();
				return Ar;
			}
		}
	}

	Ar << Header.Guid;
	Ar << Header.EngineVersion;

	if (Ar.CustomVer(FReplayCustomVersion::Guid) >= FReplayCustomVersion::SavePackageVersionUE)
	{
		Ar << Header.PackageVersionUE;
		Ar << Header.PackageVersionLicenseeUE;
	}
	else
	{
		if (Ar.IsLoading())
		{
			// Fix up for LWC compatibility.
			// Vectors were using operator<< to serialize in some network cases (instead of
			// the NetSerialize function), but this operator uses EUnrealEngineObjectUE5Version
			// and not FEngineNetworkCustomVersion for versioning. Therefore, pre-LWC replays
			// that did not save the EUnrealEngineObjectUE5Version will try to read doubles
			// from these vectors instead of floats.
			//
			// However, FEngineNetworkCustomVersion::SerializeDoubleVectorsAsDoubles was
			// added around the same time as EUnrealEngineObjectUE5Version::LARGE_WORLD_COORDINATES,
			// so we use the network version as an approximation the package version to allow
			// most pre-LWC replays to play back correctly. The compromise is that any replays recorded
			// between when FEngineNetworkCustomVersion::SerializeDoubleVectorsAsDoubles and LARGE_WORLD_COORDINATES
			// were added won't play back correctly since they didn't store accurate version information.
			if (Ar.EngineNetVer() < FEngineNetworkCustomVersion::SerializeDoubleVectorsAsDoubles)
			{
				// If the replay was recorded before vectors were serialized with LWC, and before replays
				// saved the UE package version, set the package version to one before LARGE_WORLD_COORDINATES
				// so that vectors will be read properly by operator<<.
				Header.PackageVersionUE = FPackageFileVersion(VER_LATEST_ENGINE_UE4, EUnrealEngineObjectUE5Version::OPTIONAL_RESOURCES);
			}
		}
	}

	Ar << Header.LevelNamesAndTimes;
	Ar << Header.HeaderFlags;
	Ar << Header.GameSpecificData;

	if (Ar.CustomVer(FReplayCustomVersion::Guid) >= FReplayCustomVersion::RecordingMetadata)
	{
		Ar << Header.MinRecordHz;
		Ar << Header.MaxRecordHz;

		Ar << Header.FrameLimitInMS;
		Ar << Header.CheckpointLimitInMS;

		Ar << Header.Platform;
		Ar << Header.BuildConfig;
		Ar << Header.BuildTarget;
	}

	return Ar;
}

uint32 FOverridableReplayVersionData::GetCustomVersion(const FGuid& VersionGuid) const
{
	const FCustomVersion* CustomVer = CustomVersions.GetVersion(VersionGuid);
	return CustomVer != nullptr ? CustomVer->Version : 0;
}
