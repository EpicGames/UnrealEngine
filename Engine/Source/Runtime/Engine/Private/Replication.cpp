// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	Replication.cpp: Engine actor replication implementation
=============================================================================*/

#include "Camera/PlayerCameraManager.h"

//
// Static variables for networking.
//

/** @hack: saves and restores fade state for a PC when it goes out of scope
 * used for fade track hack below */
struct FSavedFadeState
{
public:
	FSavedFadeState(APlayerCameraManager* InCamera)
		: Camera(InCamera), bEnableFading(InCamera->bEnableFading), FadeAmount(InCamera->FadeAmount), FadeTimeRemaining(InCamera->FadeTimeRemaining)
	{}
	~FSavedFadeState()
	{
		Camera->bEnableFading = bEnableFading;
		Camera->FadeAmount = FadeAmount;
		Camera->FadeTimeRemaining = FadeTimeRemaining;
	}
private:
	APlayerCameraManager* Camera;
	bool bEnableFading;
	float FadeAmount;
	float FadeTimeRemaining;
};

