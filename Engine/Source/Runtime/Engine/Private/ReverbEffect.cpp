// Copyright Epic Games, Inc. All Rights Reserved.

#include "Sound/ReverbEffect.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ReverbEffect)

UReverbEffect::UReverbEffect(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bBypassEarlyReflections = true;
	bBypassLateReflections = false;
	Density = 1.0f;
	Diffusion = 1.0f;
	Gain = 0.32f;
	GainHF = 0.89f;
	DecayTime = 1.49f;
	DecayHFRatio = 0.83f;
	ReflectionsGain = 0.05f;
	ReflectionsDelay = 0.007f;
	LateGain = 1.26f;
	LateDelay = 0.011f;
	AirAbsorptionGainHF = 0.994f;

#if WITH_EDITORONLY_DATA
	bChanged = false;
#endif
}

#if WITH_EDITORONLY_DATA
void UReverbEffect::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	bChanged = true;
}
#endif

