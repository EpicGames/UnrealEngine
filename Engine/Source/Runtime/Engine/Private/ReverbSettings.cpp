// Copyright Epic Games, Inc. All Rights Reserved.

#include "Sound/ReverbSettings.h"

#include "Sound/ReverbEffect.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ReverbSettings)


bool FReverbSettings::operator==(const FReverbSettings& Other) const
{
	return (bApplyReverb == Other.bApplyReverb
		&& ReverbEffect == Other.ReverbEffect
		&& ReverbPluginEffect == Other.ReverbPluginEffect
		&& Volume == Other.Volume
		&& FadeTime == Other.FadeTime);
}

#if WITH_EDITORONLY_DATA
void FReverbSettings::PostSerialize(const FArchive& Ar)
{
	if (Ar.UEVer() < VER_UE4_REVERB_EFFECT_ASSET_TYPE)
	{
		FString ReverbAssetName;
		switch (ReverbType_DEPRECATED)
		{
		case REVERB_Default:
			// No replacement asset for default reverb type
			return;

		case REVERB_Bathroom:
			ReverbAssetName = TEXT("/Engine/EngineSounds/ReverbSettings/Bathroom.Bathroom");
			break;

		case REVERB_StoneRoom:
			ReverbAssetName = TEXT("/Engine/EngineSounds/ReverbSettings/StoneRoom.StoneRoom");
			break;

		case REVERB_Auditorium:
			ReverbAssetName = TEXT("/Engine/EngineSounds/ReverbSettings/Auditorium.Auditorium");
			break;

		case REVERB_ConcertHall:
			ReverbAssetName = TEXT("/Engine/EngineSounds/ReverbSettings/ConcertHall.ConcertHall");
			break;

		case REVERB_Cave:
			ReverbAssetName = TEXT("/Engine/EngineSounds/ReverbSettings/Cave.Cave");
			break;

		case REVERB_Hallway:
			ReverbAssetName = TEXT("/Engine/EngineSounds/ReverbSettings/Hallway.Hallway");
			break;

		case REVERB_StoneCorridor:
			ReverbAssetName = TEXT("/Engine/EngineSounds/ReverbSettings/StoneCorridor.StoneCorridor");
			break;

		case REVERB_Alley:
			ReverbAssetName = TEXT("/Engine/EngineSounds/ReverbSettings/Alley.Alley");
			break;

		case REVERB_Forest:
			ReverbAssetName = TEXT("/Engine/EngineSounds/ReverbSettings/Forest.Forest");
			break;

		case REVERB_City:
			ReverbAssetName = TEXT("/Engine/EngineSounds/ReverbSettings/City.City");
			break;

		case REVERB_Mountains:
			ReverbAssetName = TEXT("/Engine/EngineSounds/ReverbSettings/Mountains.Mountains");
			break;

		case REVERB_Quarry:
			ReverbAssetName = TEXT("/Engine/EngineSounds/ReverbSettings/Quarry.Quarry");
			break;

		case REVERB_Plain:
			ReverbAssetName = TEXT("/Engine/EngineSounds/ReverbSettings/Plain.Plain");
			break;

		case REVERB_ParkingLot:
			ReverbAssetName = TEXT("/Engine/EngineSounds/ReverbSettings/ParkingLot.ParkingLot");
			break;

		case REVERB_SewerPipe:
			ReverbAssetName = TEXT("/Engine/EngineSounds/ReverbSettings/SewerPipe.SewerPipe");
			break;

		case REVERB_Underwater:
			ReverbAssetName = TEXT("/Engine/EngineSounds/ReverbSettings/Underwater.Underwater");
			break;

		case REVERB_SmallRoom:
			ReverbAssetName = TEXT("/Engine/EngineSounds/ReverbSettings/SmallRoom.SmallRoom");
			break;

		case REVERB_MediumRoom:
			ReverbAssetName = TEXT("/Engine/EngineSounds/ReverbSettings/MediumRoom.MediumRoom");
			break;

		case REVERB_LargeRoom:
			ReverbAssetName = TEXT("/Engine/EngineSounds/ReverbSettings/LargeRoom.LargeRoom");
			break;

		case REVERB_MediumHall:
			ReverbAssetName = TEXT("/Engine/EngineSounds/ReverbSettings/MediumHall.MediumHall");
			break;

		case REVERB_LargeHall:
			ReverbAssetName = TEXT("/Engine/EngineSounds/ReverbSettings/LargeHall.LargeHall");
			break;

		case REVERB_Plate:
			ReverbAssetName = TEXT("/Engine/EngineSounds/ReverbSettings/Plate.Plate");
			break;

		default:
			// This should cover every type of reverb preset
			checkNoEntry();
			break;
		}

		ReverbEffect = LoadObject<UReverbEffect>(NULL, *ReverbAssetName);
		check(ReverbEffect);
	}
}
#endif // WITH_EDITORONLY_DATA
