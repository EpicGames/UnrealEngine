// Copyright Epic Games, Inc. All Rights Reserved.

#include "Engine/RuntimeOptionsBase.h"
#include "Misc/ConfigContext.h"
#include "Misc/CommandLine.h"
#include "HAL/IConsoleManager.h"
#include "UObject/UnrealType.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RuntimeOptionsBase)

DEFINE_LOG_CATEGORY_STATIC(LogRuntimeOptionsBase, Log, All);

#define UE_RUNTIMEOPTIONSBASE_SUPPORT_CVARS !UE_BUILD_SHIPPING
#define UE_RUNTIMEOPTIONSBASE_SUPPORT_COMMANDLINE !UE_BUILD_SHIPPING

URuntimeOptionsBase::URuntimeOptionsBase()
{
	OptionCommandPrefix = TEXT("ro");
}

void URuntimeOptionsBase::InitializeRuntimeOptions()
{
	check(HasAnyFlags(RF_ClassDefaultObject));
	check(!OptionCommandPrefix.IsEmpty());

	if (!GetClass()->HasAnyClassFlags(CLASS_Abstract))
	{
		// Verify that the user hasn't created a hierarchy of non-abstract versions (this won't work with, e.g., cvar setting as only one of them could be registered)
		int32 NumNonAbstractVersions = 0;
		for (UClass* CurrentClass = GetClass(); CurrentClass != URuntimeOptionsBase::StaticClass(); CurrentClass = CurrentClass->GetSuperClass())
		{
			if (!CurrentClass->HasAnyClassFlags(CLASS_Abstract))
			{
				++NumNonAbstractVersions;
			}
		}
		ensureAlwaysMsgf(NumNonAbstractVersions == 1, TEXT("Error in %s hierarchy; properties should only be introduced in leaf non-abstract subclasses of URuntimeOptionsBase"), *GetClass()->GetName());

		// Load the .ini file if it hasn't already been loaded
		FConfigContext::ReadIntoGConfig().Load(*GetClass()->ClassConfigName.ToString());
	
		// Apply command line overrides and expose properties as console variables
		ApplyCommandlineOverrides();
		RegisterSupportedConsoleVariables(false);
	}
}

void URuntimeOptionsBase::ApplyCommandlineOverrides()
{
#if UE_RUNTIMEOPTIONSBASE_SUPPORT_COMMANDLINE
	// In non-shipping builds check the commandline for overrides and register variables with the console
	for (const FProperty* Property : TFieldRange<const FProperty>(GetClass()))
	{
		const FString FullyQualifiedName = FString::Printf(TEXT("%s.%s"), *OptionCommandPrefix, *Property->GetName());

		FString CommandLineOverride;
		if (FParse::Value(FCommandLine::Get(), *FString::Printf(TEXT("%s="), *FullyQualifiedName), /*out*/ CommandLineOverride))
		{
			Property->ImportText_InContainer(*CommandLineOverride, this, this, PPF_None);
		}
	}
#endif
}

void URuntimeOptionsBase::RegisterSupportedConsoleVariables(bool bDuringReload)
{
#if UE_RUNTIMEOPTIONSBASE_SUPPORT_CVARS
	for (const FProperty* Property : TFieldRange<const FProperty>(GetClass()))
	{
		const FString FullyQualifiedName = FString::Printf(TEXT("%s.%s"), *OptionCommandPrefix, *Property->GetName());

		uint8* DataPtr = Property->ContainerPtrToValuePtr<uint8>(this, 0);

		FString DisplayName;
#if WITH_EDITOR
		DisplayName = Property->GetMetaData("DisplayName");
#endif

		if (bDuringReload)
		{
			IConsoleManager::Get().UnregisterConsoleObject(*FullyQualifiedName, /*bKeepState=*/ false);
		}
		
		if (IConsoleObject* ExistingObj = IConsoleManager::Get().FindConsoleObject(*FullyQualifiedName, /*bTrackFrequentCalls=*/ false))
		{
			UE_LOGF(LogRuntimeOptionsBase, Error, "A conflicting CVar '%ls' already exists when trying to register a runtime option for the property %ls%ls::%ls",
				*FullyQualifiedName,
				GetClass()->GetPrefixCPP(),
				*GetClass()->GetName(),
				*Property->GetName());
		}
		else
		{
			if (const FNumericProperty* NumericProperty = CastField<const FNumericProperty>(Property))
			{
				if (NumericProperty->IsFloatingPoint())
				{
					IConsoleManager::Get().RegisterConsoleVariableRef(*FullyQualifiedName, *(float*)DataPtr, *DisplayName, ECVF_Default);
				}
				else if (NumericProperty->IsInteger())
				{
					IConsoleManager::Get().RegisterConsoleVariableRef(*FullyQualifiedName, *(int32*)DataPtr, *DisplayName, ECVF_Default);
				}
			}
			else if (const FBoolProperty* BoolProperty = CastField<FBoolProperty>(Property))
			{
				IConsoleManager::Get().RegisterConsoleVariableRef(*FullyQualifiedName, *(bool*)DataPtr, *DisplayName, ECVF_Default);
			}
			else if (const FStrProperty* StringProperty = CastField<FStrProperty>(Property))
			{
				IConsoleManager::Get().RegisterConsoleVariableRef(*FullyQualifiedName, *(FString*)DataPtr, *DisplayName, ECVF_Default);
			}
		}
	}
#endif
}

void URuntimeOptionsBase::PostInitProperties()
{
	Super::PostInitProperties();
	if (HasAnyFlags(RF_ClassDefaultObject))
	{
		InitializeRuntimeOptions();
	}
}

void URuntimeOptionsBase::PostReloadConfig(FProperty* PropertyThatWasLoaded)
{
	// Re-apply command line options
	ApplyCommandlineOverrides();

	// CVars hold an internal cache that can be misleading, so update it
	RegisterSupportedConsoleVariables(true);

	// Print out the post-reload active values for the properties
	if (UE_LOG_ACTIVE(LogRuntimeOptionsBase, Verbose))
	{
		UE_LOGF(LogRuntimeOptionsBase, Verbose, "After %ls%ls::PostReloadConfig:", GetClass()->GetPrefixCPP(), *GetClass()->GetName());
		for (const FProperty* Property : TFieldRange<const FProperty>(GetClass()))
		{
			FString	Value;
			Property->ExportText_InContainer(0, Value, this, this, this, 0);

			UE_LOGF(LogRuntimeOptionsBase, Verbose, "\t%ls%ls::%ls = %ls", GetClass()->GetPrefixCPP(), *GetClass()->GetName(), *Property->GetName(), *Value);
		}
	}
}

