// Copyright Epic Games, Inc. All Rights Reserved.

#include "Engine/SCS_Node.h"
#include "EngineLogs.h"
#include "UObject/LinkerLoad.h"
#include "Engine/World.h"
#include "Engine/InheritableComponentHandler.h"
#include "StaticMeshResources.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SCS_Node)

//////////////////////////////////////////////////////////////////////////
// USCS_Node

USCS_Node::USCS_Node(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
#if WITH_EDITORONLY_DATA
	bIsNative_DEPRECATED = false;
#endif

	bIsParentComponentNative = false;

#if WITH_EDITOR
	EditorComponentInstance = nullptr;
#endif
}

UActorComponent* USCS_Node::GetActualComponentTemplate(UBlueprintGeneratedClass* ActualBPGC) const
{
	UActorComponent* OverridenComponentTemplate = nullptr;
	static const FBoolConfigValueHelper EnableInheritableComponents(TEXT("Kismet"), TEXT("bEnableInheritableComponents"), GEngineIni);
	if (EnableInheritableComponents)
	{
		const USimpleConstructionScript* SCS = GetSCS();
		if (SCS != ActualBPGC->SimpleConstructionScript)
		{
			const FComponentKey ComponentKey(this);
			
			do
			{
				UInheritableComponentHandler* InheritableComponentHandler = ActualBPGC->GetInheritableComponentHandler();
				if (InheritableComponentHandler)
				{
					OverridenComponentTemplate = InheritableComponentHandler->GetOverridenComponentTemplate(ComponentKey);
				}

				ActualBPGC = Cast<UBlueprintGeneratedClass>(ActualBPGC->GetSuperClass());

			} while (!OverridenComponentTemplate && ActualBPGC && SCS != ActualBPGC->SimpleConstructionScript);
		}
	}
	return OverridenComponentTemplate ? OverridenComponentTemplate : ToRawPtr(ComponentTemplate);
}

const FBlueprintCookedComponentInstancingData* USCS_Node::GetActualComponentTemplateData(UBlueprintGeneratedClass* ActualBPGC) const
{
	const FBlueprintCookedComponentInstancingData* OverridenComponentTemplateData = nullptr;
	static const FBoolConfigValueHelper EnableInheritableComponents(TEXT("Kismet"), TEXT("bEnableInheritableComponents"), GEngineIni);
	if (EnableInheritableComponents)
	{
		const USimpleConstructionScript* SCS = GetSCS();
		if (SCS != ActualBPGC->SimpleConstructionScript)
		{
			const FComponentKey ComponentKey(this);
			
			do
			{
				UInheritableComponentHandler* InheritableComponentHandler = ActualBPGC->GetInheritableComponentHandler();
				if (InheritableComponentHandler)
				{
					OverridenComponentTemplateData = InheritableComponentHandler->GetOverridenComponentTemplateData(ComponentKey);
				}

				ActualBPGC = Cast<UBlueprintGeneratedClass>(ActualBPGC->GetSuperClass());

			} while (!OverridenComponentTemplateData && ActualBPGC && SCS != ActualBPGC->SimpleConstructionScript);
		}
	}

	return OverridenComponentTemplateData ? OverridenComponentTemplateData : &CookedComponentInstancingData;
}

UActorComponent* USCS_Node::ExecuteNodeOnActor(AActor* Actor, USceneComponent* ParentComponent, const FTransform* RootTransform, const FRotationConversionCache* RootRelativeRotationCache, bool bIsDefaultTransform, ESpawnActorScaleMethod TransformScaleMethod)
{
	check(Actor != nullptr);
	check(IsValid(ParentComponent) || (RootTransform != nullptr)); // must specify either a parent component or a world transform

	// Create a new component instance based on the template
	UActorComponent* NewActorComp = nullptr;
	UBlueprintGeneratedClass* ActualBPGC = CastChecked<UBlueprintGeneratedClass>(Actor->GetClass());
	const FBlueprintCookedComponentInstancingData* ActualComponentTemplateData = ActualBPGC->UseFastPathComponentInstancing() ? GetActualComponentTemplateData(ActualBPGC) : nullptr;
	if (ActualComponentTemplateData && ActualComponentTemplateData->bHasValidCookedData
		&& ensureMsgf(ActualComponentTemplateData->ComponentTemplateClass != nullptr, TEXT("SCS fast path (%s.%s): Cooked data is valid, but runtime support data is not initialized. Using the slow path instead."), *ActualBPGC->GetName(), *InternalVariableName.ToString()))
	{
		// Use cooked instancing data if valid (fast path).
		NewActorComp = Actor->CreateComponentFromTemplateData(ActualComponentTemplateData, InternalVariableName);
	}
	else if (UActorComponent* ActualComponentTemplate = GetActualComponentTemplate(ActualBPGC))
	{
		NewActorComp = Actor->CreateComponentFromTemplate(ActualComponentTemplate, InternalVariableName);
	}

	if(NewActorComp != nullptr)
	{
		NewActorComp->CreationMethod = EComponentCreationMethod::SimpleConstructionScript;

		// SCS created components are net addressable
		NewActorComp->SetNetAddressable();

		if (!NewActorComp->HasBeenCreated())
		{
			// Call function to notify component it has been created
			NewActorComp->OnComponentCreated();
		}

		// Special handling for scene components
		USceneComponent* NewSceneComp = Cast<USceneComponent>(NewActorComp);
		if (NewSceneComp != nullptr)
		{
			// Only register scene components if the world is initialized
			UWorld* World = Actor->GetWorld();
			bool bRegisterComponent = World && World->bIsWorldInitialized;

			// If NULL is passed in, we are the root, so set transform and assign as RootComponent on Actor, similarly if the 
			// NewSceneComp is the ParentComponent then we are the root component. This happens when the root component is recycled
			// by StaticAllocateObject.
			if (!IsValid(ParentComponent) || ParentComponent == NewSceneComp)
			{
				FTransform WorldTransform = *RootTransform;
				switch(TransformScaleMethod)
				{
				case ESpawnActorScaleMethod::OverrideRootScale:
				case ESpawnActorScaleMethod::SelectDefaultAtRuntime:
					// Use the provided transform and ignore the root component
					break;
				case ESpawnActorScaleMethod::MultiplyWithRoot:
					WorldTransform = NewSceneComp->GetRelativeTransform() * WorldTransform;
					break;
				}
				
				if(bIsDefaultTransform)
				{
					// Note: We use the scale vector from the component template when spawning (to match what happens with a native root). This
					// does NOT occur when this component is instanced as part of dynamically spawning a Blueprint class in a cooked build (i.e.
					// 'bIsDefaultTransform' will be 'false' in that situation).
					WorldTransform.SetScale3D(NewSceneComp->GetRelativeScale3D());
				}

				if (RootRelativeRotationCache)
				{	// Enforces using the same rotator as much as possible.
					NewSceneComp->SetRelativeRotationCache(*RootRelativeRotationCache);
				}

				NewSceneComp->SetWorldTransform(WorldTransform);
				Actor->SetRootComponent(NewSceneComp);

				// This will be true if we deferred the RegisterAllComponents() call at spawn time. In that case, we can call it now since we have set a scene root.
				if (Actor->HasDeferredComponentRegistration() && bRegisterComponent)
				{
					// Register the root component along with any components whose registration may have been deferred pending SCS execution in order to establish a root.
					Actor->RegisterAllComponents();
				}
			}
			// Otherwise, attach to parent component passed in
			else
			{
				NewSceneComp->SetupAttachment(ParentComponent, AttachToName);
			}

			// Register SCS scene components now (if necessary). Non-scene SCS component registration is deferred until after SCS execution, as there can be dependencies on the scene hierarchy.
			if (bRegisterComponent)
			{
				FStaticMeshComponentBulkReregisterContext* ReregisterContext = Cast<USimpleConstructionScript>(GetOuter())->GetReregisterContext();
				if (ReregisterContext)
				{
					ReregisterContext->AddConstructedComponent(NewSceneComp);
				}
				USimpleConstructionScript::RegisterInstancedComponent(NewSceneComp);
			}
		}

		// If we want to save this to a property, do it here
		FName VarName = InternalVariableName;
		if (VarName != NAME_None)
		{
			UClass* ActorClass = Actor->GetClass();
			if (FObjectPropertyBase* Prop = FindFProperty<FObjectPropertyBase>(ActorClass, VarName))
			{
				// If it is null we don't really know what's going on, but make it behave as it did before the bug fix
				if (Prop->PropertyClass == nullptr || NewActorComp->IsA(Prop->PropertyClass))
				{
					Prop->SetObjectPropertyValue_InContainer(Actor, NewActorComp);
				}
				else
				{
					UE_LOGF(LogBlueprint, Log, "ExecuteNodeOnActor: Property '%ls' on '%ls' is of type '%ls'. Could not assign '%ls' to it.", *VarName.ToString(), *Actor->GetName(), *Prop->PropertyClass->GetName(), *NewActorComp->GetName());
				}
			}
			else
			{
				UE_LOGF(LogBlueprint, Log, "ExecuteNodeOnActor: Couldn't find property '%ls' on '%ls'", *VarName.ToString(), *Actor->GetName());
#if WITH_EDITOR
				// If we're constructing editable components in the SCS editor, set the component instance corresponding to this node for editing purposes
				USimpleConstructionScript* SCS = GetSCS();
				if(SCS != nullptr && (SCS->IsConstructingEditorComponents() || SCS->GetComponentEditorActorInstance() == Actor))
				{
					EditorComponentInstance = NewSceneComp;
				}
#endif
			}
		}

		// Determine the parent component for our children (it's still our parent if we're a non-scene component)
		USceneComponent* ParentSceneComponentOfChildren = (NewSceneComp != nullptr) ? NewSceneComp : ParentComponent;

		// If we made a component, go ahead and process our children
		for (int32 NodeIdx = 0; NodeIdx < ChildNodes.Num(); NodeIdx++)
		{
			USCS_Node* Node = ChildNodes[NodeIdx];
			check(Node != nullptr);
			Node->ExecuteNodeOnActor(Actor, ParentSceneComponentOfChildren, nullptr, nullptr, false);
		}
	}

	return NewActorComp;
}

TArray<USCS_Node*> USCS_Node::GetAllNodes()
{
	TArray<USCS_Node*> AllNodes;

	//  first add ourself
	AllNodes.Add(this);

	// then add each child (including all their children)
	for(int32 ChildIdx=0; ChildIdx<ChildNodes.Num(); ChildIdx++)
	{
		USCS_Node* ChildNode = ChildNodes[ChildIdx];
		check(ChildNode != NULL);
		AllNodes.Append( ChildNode->GetAllNodes() );
	}

	return AllNodes;
}

void USCS_Node::AddChildNode(USCS_Node* InNode, bool bAddToAllNodes)
{
	if (InNode != NULL && !ChildNodes.Contains(InNode))
	{
		Modify();

		ChildNodes.Add(InNode);
		if (bAddToAllNodes)
		{
			FSCSAllNodesHelper::Add(GetSCS(), InNode);
		}
	}
}

void USCS_Node::RemoveChildNodeAt(int32 ChildIndex, bool bRemoveFromAllNodes)
{
	if (ChildIndex >= 0 && ChildIndex < ChildNodes.Num())
	{
		Modify();

		USCS_Node* ChildNode = ChildNodes[ChildIndex];
		ChildNodes.RemoveAt(ChildIndex);
		if (bRemoveFromAllNodes)
		{
			FSCSAllNodesHelper::Remove(GetSCS(), ChildNode);
		}
	}
}

void USCS_Node::RemoveChildNode(USCS_Node* InNode, bool bRemoveFromAllNodes)
{
	Modify();
	if (ChildNodes.Remove(InNode) != INDEX_NONE && bRemoveFromAllNodes)
	{
		FSCSAllNodesHelper::Remove(GetSCS(), InNode);
	}
}

void USCS_Node::MoveChildNodes(USCS_Node* SourceNode, int32 InsertLocation)
{
	if (SourceNode)
	{
		Modify();
		SourceNode->Modify();

		USimpleConstructionScript* SourceSCS = SourceNode->GetSCS();
		USimpleConstructionScript* MySCS = GetSCS();
		if (SourceSCS != MySCS)
		{
			for (USCS_Node* SCSNode : SourceNode->ChildNodes)
			{
				FSCSAllNodesHelper::Remove(SourceSCS, SCSNode);
				FSCSAllNodesHelper::Add(MySCS, SCSNode);
			}
		}
		if (InsertLocation == INDEX_NONE)
		{
			ChildNodes.Append(SourceNode->ChildNodes);
		}
		else
		{
			ChildNodes.Insert(SourceNode->ChildNodes, InsertLocation);
		}
		SourceNode->ChildNodes.Empty();
	}
}

TArray<const USCS_Node*> USCS_Node::GetAllNodes() const
{
	TArray<const USCS_Node*> AllNodes;

	//  first add ourself
	AllNodes.Add(this);

	// then add each child (including all their children)
	for(int32 ChildIdx=0; ChildIdx<ChildNodes.Num(); ChildIdx++)
	{
		const USCS_Node* ChildNode = ChildNodes[ChildIdx];
		check(ChildNode != NULL);
		AllNodes.Append( ChildNode->GetAllNodes() );
	}

	return AllNodes;
}

bool USCS_Node::IsChildOf(USCS_Node* TestParent)
{
	TArray<USCS_Node*> AllNodes;
	if(TestParent != NULL)
	{
		AllNodes = TestParent->GetAllNodes();
	}
	return AllNodes.Contains(this);
}

void USCS_Node::PreloadChain()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(USCS_Node::PreloadChain);

	ConditionalPreload();

	if (ComponentTemplate)
	{
		ComponentTemplate->ConditionalPreload();

		TArray<UObject*> Children;
		GetObjectsWithOuter(ComponentTemplate, Children, EGetObjectsFlags::IncludeNestedObjects);

		for (UObject* Obj : Children)
		{
			Obj->ConditionalPreload();
		}
	}

	for( decltype(ChildNodes)::TIterator ChildIt(ChildNodes); ChildIt; ++ChildIt )
	{
		USCS_Node* CurrentChild = *ChildIt;
		if( CurrentChild )
		{
			CurrentChild->PreloadChain();
		}
	}
}

bool USCS_Node::IsRootNode() const
{
	USimpleConstructionScript* SCS = GetSCS();
	return(SCS->GetRootNodes().Contains(const_cast<USCS_Node*>(this)));
}

void USCS_Node::RenameComponentTemplate(UActorComponent* ComponentTemplate, const FName& NewName)
{
	if (ComponentTemplate != nullptr && ComponentTemplate->HasAllFlags(RF_ArchetypeObject))
	{
		// Gather all instances of the template (archetype)
		TArray<UObject*> ArchetypeInstances;
		ComponentTemplate->GetArchetypeInstances(ArchetypeInstances);

		// Rename the component template (archetype) - note that this can be called during compile-on-load, so we include the flag not to reset the BPGC's package loader.
		const FString NewComponentName = NewName.ToString();
		ComponentTemplate->Rename(*(NewComponentName + USimpleConstructionScript::ComponentTemplateNameSuffix), nullptr, REN_DontCreateRedirectors | REN_AllowPackageLinkerMismatch);

		// Rename all component instances to match the updated variable name
		for (UObject* ArchetypeInstance : ArchetypeInstances)
		{
			// Recursively handle inherited component template overrides. In the SCS case, this is because we must also handle these before the SCS key's variable name is changed.
			if (ArchetypeInstance->HasAllFlags(RF_ArchetypeObject | RF_InheritableComponentTemplate))
			{
				RenameComponentTemplate(CastChecked<UActorComponent>(ArchetypeInstance), NewName);
			}
			else
			{
				// If this is an instanced component (i.e. owned by an Actor), ensure that we have no conflict with another instanced component belonging to the same Actor instance.
				if (AActor* Actor = Cast<AActor>(ArchetypeInstance->GetOuter()))
				{
					Actor->CheckComponentInstanceName(NewName);
				}

				ArchetypeInstance->Rename(*NewComponentName, nullptr, REN_DontCreateRedirectors | REN_AllowPackageLinkerMismatch);
			}
		}
	}
}

void USCS_Node::SetVariableName(const FName& NewName, bool bRenameTemplate)
{
	// We need to ensure that component object names stay in sync with the variable name; this is done for 2 reasons:
	//	1) This ensures existing instances can successfully route back to the archetype (template) object through the variable name.
	//	2) This prevents new SCS nodes for the same component type from recycling an existing template with the original (base) name.
	if (bRenameTemplate && ComponentTemplate != nullptr)
	{
		// This must be called BEFORE we change the internal variable name; otherwise it will fail to find any instances of the archetype!
		RenameComponentTemplate(ComponentTemplate, NewName);
	}

	InternalVariableName = NewName;
}

#if WITH_EDITOR
void USCS_Node::NameWasModified()
{
	OnNameChangedExternal.ExecuteIfBound(InternalVariableName);
}

void USCS_Node::SetOnNameChanged( const FSCSNodeNameChanged& OnChange )
{
	OnNameChangedExternal = OnChange;
}
#endif

int32 USCS_Node::FindMetaDataEntryIndexForKey(const FName Key) const
{
	for(int32 i=0; i<MetaDataArray.Num(); i++)
	{
		if(MetaDataArray[i].DataKey == Key)
		{
			return i;
		}
	}
	return INDEX_NONE;
}

const FString& USCS_Node::GetMetaData(const FName Key) const
{
	int32 EntryIndex = FindMetaDataEntryIndexForKey(Key);
	check(EntryIndex != INDEX_NONE);
	return MetaDataArray[EntryIndex].DataValue;
}

void USCS_Node::SetMetaData(const FName Key, FString Value)
{
	int32 EntryIndex = FindMetaDataEntryIndexForKey(Key);
	if(EntryIndex != INDEX_NONE)
	{
		MetaDataArray[EntryIndex].DataValue = MoveTemp(Value);
	}
	else
	{
		MetaDataArray.Emplace( FBPVariableMetaDataEntry(Key, MoveTemp(Value)) );
	}
}

void USCS_Node::RemoveMetaData(const FName Key)
{
	int32 EntryIndex = FindMetaDataEntryIndexForKey(Key);
	if(EntryIndex != INDEX_NONE)
	{
		MetaDataArray.RemoveAt(EntryIndex);
	}
}

void USCS_Node::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

	if (Ar.IsLoading())
	{
		if (Ar.IsPersistent() && !Ar.HasAnyPortFlags(PPF_Duplicate | PPF_DuplicateForPIE))
		{
			// Fix up the component class property, if it has not already been set.
			// Note: This is done here, instead of in PostLoad(), because it needs to be set before Blueprint class compilation.
			if (ComponentClass == nullptr && ComponentTemplate != nullptr)
			{
				ComponentClass = ComponentTemplate->GetClass();
			}

			// Only "override" template objects created/referenced by the ICH should have this flag set.
			// Older versions may have been saved with this flag incorrectly set on the default root node.
			if (ComponentTemplate != nullptr)
			{
				ComponentTemplate->ClearFlags(RF_InheritableComponentTemplate);
			}
		}
	}
}

void USCS_Node::PostLoad()
{
	Super::PostLoad();

#if WITH_EDITOR
	ValidateGuid();
#endif
}


#if WITH_EDITOR
void USCS_Node::SetParent(USCS_Node* InParentNode)
{
	ensure(InParentNode);
	USimpleConstructionScript* ParentSCS = InParentNode ? InParentNode->GetSCS() : nullptr;
	ensure(ParentSCS);
	UClass* ParentBlueprintGeneratedClass = ParentSCS ? ParentSCS->GetOwnerClass() : nullptr;

	if (ParentBlueprintGeneratedClass && InParentNode)
	{
		const FName NewParentComponentOrVariableName = InParentNode->GetVariableName();
		const FName NewParentComponentOwnerClassName = ParentBlueprintGeneratedClass->GetFName();

		// Only modify if it differs from current
		if (bIsParentComponentNative
			|| ParentComponentOrVariableName != NewParentComponentOrVariableName
			|| ParentComponentOwnerClassName != NewParentComponentOwnerClassName)
		{
			Modify();

			bIsParentComponentNative = false;
			ParentComponentOrVariableName = NewParentComponentOrVariableName;
			ParentComponentOwnerClassName = NewParentComponentOwnerClassName;
		}
	}
}

void USCS_Node::SetParent(const USceneComponent* InParentComponent)
{
	check(InParentComponent != NULL);

	const FName NewParentComponentOrVariableName = InParentComponent->GetFName();
	const FName NewParentComponentOwnerClassName = NAME_None;

	// Only modify if it differs from current
	if(!bIsParentComponentNative
		|| ParentComponentOrVariableName != NewParentComponentOrVariableName
		|| ParentComponentOwnerClassName != NewParentComponentOwnerClassName)
	{
		Modify();

		bIsParentComponentNative = true;
		ParentComponentOrVariableName = NewParentComponentOrVariableName;
		ParentComponentOwnerClassName = NewParentComponentOwnerClassName;
	}
}

USceneComponent* USCS_Node::GetParentComponentTemplate(UBlueprint* InBlueprint) const
{
	check(InBlueprint && InBlueprint->GeneratedClass);
	return GetParentComponentTemplate(CastChecked<UBlueprintGeneratedClass>(InBlueprint->GeneratedClass));
}

USceneComponent* USCS_Node::GetParentComponentTemplate(UBlueprintGeneratedClass* BPGC) const
{
	USceneComponent* ParentComponentTemplate = nullptr;
	if(ParentComponentOrVariableName != NAME_None)
	{
		check(BPGC);

		// If the parent component template is found in the 'Components' array of the CDO (i.e. native)
		if(bIsParentComponentNative)
		{
			// Access the Blueprint CDO
			AActor* CDO = BPGC->GetDefaultObject<AActor>();
			if(CDO != nullptr)
			{
				// Find the component template in the CDO that matches the specified name
				for (UActorComponent* ActorComp : CDO->GetComponents())
				{
					USceneComponent* CompTemplate = Cast<USceneComponent>(ActorComp);
					if (CompTemplate && CompTemplate->GetFName() == ParentComponentOrVariableName)
					{
						// Found a match; this is our parent, we're done
						ParentComponentTemplate = CompTemplate;
						break;
					}
				}
			}
		}
		// Otherwise the parent component template is found in a parent Blueprint's SCS tree (i.e. non-native)
		else
		{
			// Get the Blueprint hierarchy
			TArray<UBlueprintGeneratedClass*> ParentBPStack;
			UBlueprint::GetBlueprintHierarchyFromClass(BPGC, ParentBPStack);

			// Find the parent Blueprint in the hierarchy
			for (int32 StackIndex = ParentBPStack.Num() - 1; StackIndex > 0 && !ParentComponentTemplate; --StackIndex)
			{
				UBlueprintGeneratedClass* ParentBPGC = ParentBPStack[StackIndex];
				if (ParentBPGC
					&& ParentBPGC->SimpleConstructionScript
					&& ParentBPGC->GetFName() == ParentComponentOwnerClassName)
				{
					// Find the SCS node with a variable name that matches the specified name
					for (USCS_Node* Node : ParentBPGC->SimpleConstructionScript->GetAllNodes())
					{
						USceneComponent* CompTemplate = Cast<USceneComponent>(Node->ComponentTemplate);
						if (CompTemplate && Node->GetVariableName() == ParentComponentOrVariableName)
						{
							// Found a match; this is our parent, we're done
							ParentComponentTemplate = Cast<USceneComponent>(Node->GetActualComponentTemplate(Cast<UBlueprintGeneratedClass>(BPGC)));
							break;
						}
					}
				}
			}
		}
	}

	return ParentComponentTemplate;
}

void USCS_Node::SaveToTransactionBuffer()
{
	Modify();

	for (USCS_Node* ChildNode : GetChildNodes())
	{
		if (ChildNode)
		{
			ChildNode->SaveToTransactionBuffer();
		}
	}
}

void USCS_Node::ValidateGuid()
{
	// Backward compatibility:
	// The guid for the node should be always the same (event when it's not saved). 
	// The guid is created in an deterministic way using persistent name.
	if (!VariableGuid.IsValid() && (InternalVariableName != NAME_None))
	{
		const FString HashString = InternalVariableName.ToString();
		ensure(HashString.Len());

		const uint32 BufferLength = HashString.Len() * sizeof(HashString[0]);
		uint32 HashBuffer[5];
		FSHA1::HashBuffer(*HashString, BufferLength, reinterpret_cast<uint8*>(HashBuffer));
		VariableGuid = FGuid(HashBuffer[1], HashBuffer[2], HashBuffer[3], HashBuffer[4]);
	}
}

EDataValidationResult USCS_Node::IsDataValid(FDataValidationContext& Context) const
{
	EDataValidationResult Result = Super::IsDataValid(Context);
	Result = (Result == EDataValidationResult::NotValidated) ? EDataValidationResult::Valid : Result;

	// check the component that this node represents
	if (ComponentTemplate)
	{
		EDataValidationResult ComponentResult = AsConst(*ComponentTemplate).IsDataValid(Context);
		Result = CombineDataValidationResults(Result, ComponentResult);
	}

	// check children
	for (USCS_Node* Child : ChildNodes)
	{
		if (Child)
		{
			EDataValidationResult ChildResult = Child->IsDataValid(Context);
			Result = CombineDataValidationResults(Result, ChildResult);
		}
	}

	return Result;
}

#endif // WITH_EDITOR

