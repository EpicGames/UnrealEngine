// Copyright Epic Games, Inc. All Rights Reserved.

#include "Sound/SampleBufferIO.h"
#include "Engine/Engine.h"
#include "HAL/PlatformFile.h"
#include "HAL/PlatformFileManager.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "Misc/PackageName.h"
#include "AudioDevice.h"
#include "Async/Async.h"
#include "UObject/Package.h"

namespace Audio
{
	FSoundWavePCMLoader::FSoundWavePCMLoader()
	{
	}

	void FSoundWavePCMLoader::LoadSoundWave(USoundWave* InSoundWave, TFunction<void(const USoundWave* SoundWave, const Audio::FSampleBuffer& OutSampleBuffer)> OnLoaded, bool bSynchronous /** = false */)
	{
		FAudioDeviceHandle AudioDevice = FAudioDevice::GetMainAudioDevice();

		if (!AudioDevice || !InSoundWave)
		{
			return;
		}

		FLoadingSoundWaveInfo LoadingSoundWaveInfo;

		const bool bRequestPrecache = InSoundWave->GetPrecacheState() != ESoundWavePrecacheState::Done ||
			!InSoundWave->RawPCMData ||
			InSoundWave->RawPCMDataSize == 0;
		if (bRequestPrecache)
		{
			LoadingSoundWaveInfo.Status = FLoadingSoundWaveInfo::LoadStatus::Loading;

			// Kick off a decompression/precache of the sound wave
			AudioDevice->Precache(InSoundWave, bSynchronous, false, true);
		}
		else
		{
			LoadingSoundWaveInfo.Status = FLoadingSoundWaveInfo::LoadStatus::Loaded;
		}

		if (bSynchronous)
		{
			check(InSoundWave->GetPrecacheState() == ESoundWavePrecacheState::Done);

			const Audio::DefaultUSoundWaveSampleType* RawPCMData = reinterpret_cast<const Audio::DefaultUSoundWaveSampleType*>(InSoundWave->RawPCMData);
			const int32 NumSamples = InSoundWave->RawPCMDataSize / sizeof(Audio::DefaultUSoundWaveSampleType);

			TSampleBuffer<> SampleBuffer(RawPCMData, NumSamples, InSoundWave->NumChannels, InSoundWave->GetSampleRateForCurrentPlatform());
			OnLoaded(InSoundWave, SampleBuffer);

		}
		else
		{
			LoadingSoundWaveInfo.SoundWave = InSoundWave;
			LoadingSoundWaveInfo.OnLoaded = MoveTemp(OnLoaded);

			LoadingSoundWaves.Add(LoadingSoundWaveInfo);
		}
	}


	void FSoundWavePCMLoader::Update()
	{
		for (int32 i = LoadingSoundWaves.Num() - 1; i >= 0; --i)
		{
			FLoadingSoundWaveInfo& LoadingSoundWaveInfo = LoadingSoundWaves[i];
			if (USoundWave* SoundWave = LoadingSoundWaveInfo.SoundWave)
			{
				if (SoundWave->GetPrecacheState() == ESoundWavePrecacheState::Done)
				{
					LoadingSoundWaveInfo.Status = FLoadingSoundWaveInfo::LoadStatus::Loaded;
				}

				if (LoadingSoundWaveInfo.Status == FLoadingSoundWaveInfo::LoadStatus::Loaded)
				{
					const Audio::DefaultUSoundWaveSampleType* RawPCMData = reinterpret_cast<const Audio::DefaultUSoundWaveSampleType*>(SoundWave->RawPCMData);
					const int32 NumSamples = SoundWave->RawPCMDataSize / sizeof(Audio::DefaultUSoundWaveSampleType);

					TSampleBuffer<> SampleBuffer(RawPCMData, NumSamples, SoundWave->NumChannels, SoundWave->GetSampleRateForCurrentPlatform());
					LoadingSoundWaveInfo.OnLoaded(SoundWave, SampleBuffer);
					LoadingSoundWaves.RemoveAtSwap(i, EAllowShrinking::No);
				}
			}
		}
	}

	void FSoundWavePCMLoader::AddReferencedObjects(FReferenceCollector& Collector)
	{
		for (FLoadingSoundWaveInfo& LoadingSoundWave : LoadingSoundWaves)
		{
			check(LoadingSoundWave.SoundWave);
			Collector.AddReferencedObject(LoadingSoundWave.SoundWave);
		}
	}

	FString FSoundWavePCMLoader::GetReferencerName() const
	{
		return TEXT("FSoundWavePCMLoader");
	}

	FSoundWavePCMWriter::FSoundWavePCMWriter(int32 InChunkSize)
		: CurrentSoundWave(0)
		, CurrentState(ESoundWavePCMWriterState::Idle)
		, bWasPreviouslyAddedToRoot(false)
		, ChunkSize(InChunkSize)
		, CurrentPackage(nullptr)
	{
	}


	FSoundWavePCMWriter::~FSoundWavePCMWriter()
	{
		Reset();
	}

	bool FSoundWavePCMWriter::BeginGeneratingSoundWaveFromBuffer(const TSampleBuffer<>& InSampleBuffer, USoundWave* SoundWaveToSaveTo, TFunction<void(const USoundWave*)> OnSuccess)
	{
		if (!IsDone())
		{
			UE_LOGF(LogAudio, Error, "This instance of FSoundWavePCMWriter is already processing another write operation.");
			return false;
		}

		CurrentState = ESoundWavePCMWriterState::Generating;

		// If SoundWaveToSaveTo is null, create a new object.
		if (SoundWaveToSaveTo == nullptr)
		{
			CurrentSoundWave = NewObject<USoundWave>();
			CurrentSoundWave->AddToRoot();
		}
		else
		{
			CurrentSoundWave = SoundWaveToSaveTo;
			bWasPreviouslyAddedToRoot = CurrentSoundWave->IsRooted();
			CurrentSoundWave->AddToRoot();
			// Ensure this sound wave is not currently in use:
			FAudioDeviceManager* AudioDeviceManager = GEngine->GetAudioDeviceManager();
			if (AudioDeviceManager)
			{
				AudioDeviceManager->StopSoundsUsingResource(CurrentSoundWave);
			}
		}

		CurrentBuffer = InSampleBuffer;

		// TODO: Support writing to multi-channel sound waves. For now, we mix down sound waves to stereo:
		if (CurrentBuffer.GetNumChannels() > 2)
		{
			CurrentBuffer.MixBufferToChannels(2);
		}

		CurrentOperation.Reset(new FAsyncSoundWavePCMWriterTask(this, ESoundWavePCMWriteTaskType::GenerateAndWriteSoundWave, OnSuccess));
		CurrentOperation->StartBackgroundTask();

		return true;
	}

	bool FSoundWavePCMWriter::BeginWriteToSoundWave(const FString& FileName, const TSampleBuffer<>& InSampleBuffer, FString InPath, TFunction<void(const USoundWave*)> OnSuccess, TObjectPtr<USoundWave> InSoundWaveTransformationTarget)
	{
		if (!IsDone())
		{
			UE_LOGF(LogAudio, Error, "This instance of FSoundWavePCMWriter is already processing another write operation.");
			return false;
		}

		if (!GIsEditor)
		{
			UE_LOGF(LogAudio, Error, "Writing to a SoundWave is only available in the editor.");
			return false;
		}

		CurrentState = ESoundWavePCMWriterState::Generating;

		FPaths::NormalizeDirectoryName(InPath);

		AbsoluteFilePath = TEXT("/Game/") + InPath + FString("/") + FileName;

		AbsoluteFilePath = AbsoluteFilePath.Replace(TEXT("//"), TEXT("/"), ESearchCase::CaseSensitive);

		FText InvalidPathReason;
		bool const bValidPackageName = FPackageName::IsValidLongPackageName(AbsoluteFilePath, false, &InvalidPathReason);

		if (!bValidPackageName)
		{
			UE_LOGF(LogAudio, Warning, "File path given was not valid (%ls): the recording output will be saved to a USoundWave at the root of the Content folder.", *InvalidPathReason.ToString());
			AbsoluteFilePath = TEXT("/Game/") + FString("/") + FileName;
			AbsoluteFilePath = AbsoluteFilePath.Replace(TEXT("//"), TEXT("/"), ESearchCase::CaseSensitive);
		}

		// Set up Package.
		CurrentPackage = CreatePackage( *AbsoluteFilePath);

		// Create a new USoundWave.
		CurrentSoundWave = NewObject<USoundWave>(CurrentPackage, *FileName, RF_Public | RF_Standalone | RF_Transactional, InSoundWaveTransformationTarget);
		CurrentSoundWave->AddToRoot();
		
#if WITH_EDITOR
		CurrentSoundWave->ResetInvalidTemplateArguments();

		// CurrentSoundwave::Transformations must be empty because all Transformation data will be written directly to the soundwave, not overlayed as a transformation
		check(CurrentSoundWave->Transformations.IsEmpty());
#endif //WITH_EDITOR

		CurrentBuffer = InSampleBuffer;

		// TODO: Support writing to multi-channel sound waves. For now, we mix down sound waves to stereo:
		if (CurrentBuffer.GetNumChannels() > 2)
		{
			CurrentBuffer.MixBufferToChannels(2);
		}
	
		CurrentOperation.Reset(new FAsyncSoundWavePCMWriterTask(this, ESoundWavePCMWriteTaskType::GenerateAndWriteSoundWave, [this, OnSuccess, InSoundWaveTransformationTarget](const USoundWave* InSoundWave)
		{
			if (!IsValid(CurrentSoundWave))
			{
				UE_LOGF(LogAudio, Error, "CurrentSoundWave is not valid in GenerateAndWriteSoundWave.");
				return;
			}

			// Invalidate compressed data on new soundwave so that it gets refreshed or the soundwave is not seekable
			// and causes this error: FMixerBuffer::CreateStreamingBuffer failed to StreamCompressedInfo on SoundWave
			CurrentSoundWave->InvalidateCompressedData(true); // calls CurrentSoundWave->FreeResources(), note the game thread is the audio thread in the editor

#if WITH_EDITOR
			if (InSoundWaveTransformationTarget && InSoundWaveTransformationTarget->Transformations.Num() > 0)
			{
				CurrentSoundWave->OverwriteSoundWaveDataWithTransformationData(InSoundWaveTransformationTarget->Transformations);
			}
#endif //WITH_EDITOR

			OnSuccess(InSoundWave);
		}));
		CurrentOperation->StartBackgroundTask();

		return true;
	}

	bool FSoundWavePCMWriter::PrepWavFileOutput(const TSampleBuffer<>& InSampleBuffer, const FString& FileName, const FString& FilePath)
	{
		const bool bIsRelativePath = FPaths::IsRelative(FilePath);
		if (bIsRelativePath)
		{
			AbsoluteFilePath = FPaths::ProjectSavedDir() + TEXT("BouncedWavFiles/") + FilePath;
			AbsoluteFilePath = FPaths::ConvertRelativePathToFull(AbsoluteFilePath);
		}
		else
		{
			AbsoluteFilePath = FilePath;
		}

		// Fix up any slashes
		FPaths::NormalizeDirectoryName(AbsoluteFilePath);

		// Remove any "../.." from the path.
		FPaths::CollapseRelativeDirectories(AbsoluteFilePath);

		CurrentState = ESoundWavePCMWriterState::Generating;

		if (!CreateDirectoryIfNeeded(AbsoluteFilePath))
		{
			UE_LOGF(LogAudio, Error, "Write to Wav File failed: Invalid directory path %ls", *AbsoluteFilePath);
			CurrentState = ESoundWavePCMWriterState::Failed;
			return false;
		}

		// Now append the FileName
		AbsoluteFilePath = AbsoluteFilePath + TEXT("/") + FileName + TEXT(".wav");

		CurrentBuffer = InSampleBuffer;

		return true;
	}

	bool FSoundWavePCMWriter::BeginWriteToWavFile(const TSampleBuffer<>& InSampleBuffer, const FString& FileName, const FString& FilePath, TFunction<void()> OnSuccess)
	{
		if (!IsDone())
		{
			UE_LOGF(LogAudio, Error, "This instance of FSoundWavePCMWriter is already processing another write operation.");
			return false;
		}

		if (!PrepWavFileOutput(InSampleBuffer, FileName, FilePath))
		{
			return false;
		}

		// For convenience in our async task, we only take void(USoundWave*) type lambdas. So let's wrap our void() lambda here:
		TFunction<void(const USoundWave*)> WrappedCallback = [OnSuccess](const USoundWave*)
		{
			OnSuccess();
		};

		CurrentOperation.Reset(new FAsyncSoundWavePCMWriterTask(this, ESoundWavePCMWriteTaskType::WriteWavFile, WrappedCallback));
		CurrentOperation->StartBackgroundTask();

		return true;
	}

	bool FSoundWavePCMWriter::SynchronouslyWriteToWavFile(const TSampleBuffer<>& InSampleBuffer, const FString& FileName, const FString& FilePath, FString* OutFilePathName)
	{
		if (!IsDone())
		{
			UE_LOGF(LogAudio, Error, "This instance of FSoundWavePCMWriter is already processing another write operation.");
			return false;
		}

		if (!PrepWavFileOutput(InSampleBuffer, FileName, FilePath))
		{
			return false;
		}

		CurrentOperation.Reset(new FAsyncSoundWavePCMWriterTask(this, ESoundWavePCMWriteTaskType::WriteWavFile, [](const USoundWave*){}));
		CurrentOperation->StartSynchronousTask();

		if (OutFilePathName != nullptr)
		{
			*OutFilePathName = AbsoluteFilePath;
		}

		return true;
	}

	USoundWave* FSoundWavePCMWriter::SynchronouslyWriteSoundWave(const TSampleBuffer<>& InSampleBuffer, const FString* FileName, const FString* FilePath, TObjectPtr<USoundWave> InSoundWaveTransformationTarget)
	{
		if (!IsDone())
		{
			UE_LOGF(LogAudio, Error, "This instance of FSoundWavePCMWriter is already processing another write operation.");
			return nullptr;
		}

		CurrentState = ESoundWavePCMWriterState::Generating;
		
		bool bWillWriteToDisk = false;

		if (GIsEditor && FileName != nullptr)
		{
			if (FilePath != nullptr)
			{
				AbsoluteFilePath = TEXT("/Game/") + *FilePath + FString("/") + *FileName;
			}
			else
			{
				AbsoluteFilePath = TEXT("/Game/") + *FileName;
			}


			FPaths::NormalizeDirectoryName(AbsoluteFilePath);
			AbsoluteFilePath = AbsoluteFilePath.Replace(TEXT("//"), TEXT("/"), ESearchCase::CaseSensitive);

			FText InvalidPathReason;
			bool const bValidPackageName = FPackageName::IsValidLongPackageName(AbsoluteFilePath, false, &InvalidPathReason);

			if (!bValidPackageName)
			{
				UE_LOGF(LogAudio, Warning, "File path given was not valid (%ls): the recording output will be saved to a USoundWave at the root of the Content folder.", *InvalidPathReason.ToString());
				AbsoluteFilePath = TEXT("/Game/") + *FileName;
				FPaths::NormalizeDirectoryName(AbsoluteFilePath);
				AbsoluteFilePath = AbsoluteFilePath.Replace(TEXT("//"), TEXT("/"), ESearchCase::CaseSensitive);
			}

			// Set up Package.
			CurrentPackage = CreatePackage( *AbsoluteFilePath);

			// Create a new USoundWave.
			CurrentSoundWave = NewObject<USoundWave>(CurrentPackage, **FileName, RF_Public | RF_Standalone);
			bWillWriteToDisk = true;
		}
		else
		{
			CurrentSoundWave = NewObject<USoundWave>();
		}
		

		CurrentBuffer = InSampleBuffer;

		// TODO: Support writing to multi-channel sound waves. For now, we mix down sound waves to stereo:
		if (CurrentBuffer.GetNumChannels() > 2)
		{
			CurrentBuffer.MixBufferToChannels(2);
		}

		if (bWillWriteToDisk)
		{
			CurrentOperation.Reset(new FAsyncSoundWavePCMWriterTask(this, ESoundWavePCMWriteTaskType::GenerateAndWriteSoundWave, [](const USoundWave* InSoundWave) {}));
		}
		else
		{
			CurrentOperation.Reset(new FAsyncSoundWavePCMWriterTask(this, ESoundWavePCMWriteTaskType::GenerateSoundWave, [](const USoundWave* InSoundWave) {}));
		}
		
		CurrentOperation->StartSynchronousTask();

		// Invalidate compressed data on new soundwave so that it gets refreshed or the soundwave is not seekable
		// and causes this error: FMixerBuffer::CreateStreamingBuffer failed to StreamCompressedInfo on SoundWave
		CurrentSoundWave->InvalidateCompressedData(true);


#if WITH_EDITOR
		if (InSoundWaveTransformationTarget && InSoundWaveTransformationTarget->Transformations.Num() > 0)
		{
			CurrentSoundWave->OverwriteSoundWaveDataWithTransformationData(InSoundWaveTransformationTarget->Transformations);
		}
#endif //WITH_EDITOR

		return CurrentSoundWave;
	}

	float FSoundWavePCMWriter::CheckStatus(ESoundWavePCMWriterState* OutCurrentState)
	{
		if (OutCurrentState)
		{
			*OutCurrentState = CurrentState;
		}

		return ((float)Progress.GetValue()) / (SerializedWavData.Num() + 1);
	}

	void FSoundWavePCMWriter::CancelWrite()
	{
		if (CurrentOperation.IsValid())
		{
			if (!CurrentOperation->Cancel())
			{
				CurrentOperation->EnsureCompletion(true);
			}

			CurrentOperation.Reset();
		}
		CurrentState = ESoundWavePCMWriterState::Cancelled;
	}

	bool FSoundWavePCMWriter::IsDone()
	{
		bool bCurrentOperationDone = true;
		if (CurrentOperation.IsValid())
		{
			bCurrentOperationDone = CurrentOperation->IsDone();
		}

		return ((CurrentState == ESoundWavePCMWriterState::Suceeded
			|| CurrentState == ESoundWavePCMWriterState::Failed
			|| CurrentState == ESoundWavePCMWriterState::Cancelled
			|| CurrentState == ESoundWavePCMWriterState::Idle) && bCurrentOperationDone);
	}

	void FSoundWavePCMWriter::Reset()
	{
		CancelWrite();

		if (CurrentSoundWave && CurrentSoundWave->IsValidLowLevel() && !bWasPreviouslyAddedToRoot)
		{
			CurrentSoundWave->RemoveFromRoot();
		}

		CurrentSoundWave = nullptr;
		CurrentPackage = nullptr;

		Progress.Set(0);
		CurrentState = ESoundWavePCMWriterState::Idle;
	}

	USoundWave* FSoundWavePCMWriter::GetFinishedSoundWave()
	{
		if (!IsDone())
		{
			UE_LOGF(LogAudio, Warning, "Failed to get finished soundwave: write operation currently still in progress.");
			return nullptr;
		}
		else if (CurrentState != ESoundWavePCMWriterState::Suceeded)
		{
			UE_LOGF(LogAudio, Warning, "Failed to get finished soundwave: write operation failed.");
			return nullptr;
		}

		if (CurrentSoundWave)
		{
			if (!bWasPreviouslyAddedToRoot)
			{
				CurrentSoundWave->RemoveFromRoot();
			}

			return CurrentSoundWave;
		}

		// If we did not initially create a SoundWave, we can create one here and apply the buffer to it.
		check(CurrentSoundWave == nullptr);
		CurrentSoundWave = NewObject<USoundWave>();
		ApplyBufferToSoundWave();

		return CurrentSoundWave;
	}

	void FSoundWavePCMWriter::SaveFinishedSoundWaveToPath(const FString& FileName, FString InPath)
	{
		// This is an editor only function.
		if (!GIsEditor)
		{
			UE_LOGF(LogAudio, Warning, "SoundWave assets can only be saved with the editor.");
			return;
		}
		else if (!IsDone())
		{
			UE_LOGF(LogAudio, Warning, "Failed to kick off save: write operation still in progress.");
			return;
		}
		else if (CurrentState != ESoundWavePCMWriterState::Suceeded)
		{
			UE_LOGF(LogAudio, Warning, "Failed to kick off save: write operation failed.");
			return;
		}

		if (!CreateDirectoryIfNeeded(InPath))
		{
			UE_LOGF(LogAudio, Warning, "Failed to kick off save: invalid directory %ls", *InPath);
			return;
		}

		AbsoluteFilePath = InPath + FString("/") + FileName;
		AbsoluteFilePath = AbsoluteFilePath.Replace(TEXT("//"), TEXT("/"), ESearchCase::CaseSensitive);
		SerializeSoundWaveToAsset();
	}

	// todo: this function needs to be revisited to have behavior similar to SerializeSoundWaveToAsset() in non-editor environments
	// todo: for now this only updates simple members and cleans up data and clears RawPCMData to avoid taking a deprecated decode path
	void FSoundWavePCMWriter::ApplyBufferToSoundWave()
	{
		// Since we just want to replace the PCM data to save it to disk. We don't need to compute anything platformdata related.
		CurrentSoundWave->InvalidateCompressedData(false /* bFreeResources */, false /* bRebuildStreamingChunk */);

		CurrentSoundWave->SetSampleRate(CurrentBuffer.GetSampleRate());
		CurrentSoundWave->NumChannels = CurrentBuffer.GetNumChannels();
		CurrentSoundWave->RawPCMDataSize = CurrentBuffer.GetNumSamples() * sizeof(int16);
		CurrentSoundWave->Duration = (float) CurrentBuffer.GetNumFrames() / CurrentBuffer.GetSampleRate();
		CurrentSoundWave->TotalSamples = CurrentBuffer.GetNumSamples();

		if (CurrentSoundWave->RawPCMData != nullptr)
		{
			FMemory::Free(CurrentSoundWave->RawPCMData);
			CurrentSoundWave->RawPCMData = nullptr;
		}

		CurrentSoundWave->RawPCMData = (uint8*)FMemory::Malloc(CurrentSoundWave->RawPCMDataSize);
		FMemory::Memcpy(CurrentSoundWave->RawPCMData, CurrentBuffer.GetData(), CurrentSoundWave->RawPCMDataSize);
	}

	void FSoundWavePCMWriter::SerializeSoundWaveToAsset()
	{
		check(CurrentSoundWave != nullptr);
		CurrentState = ESoundWavePCMWriterState::Generating;

		if (CurrentBuffer.GetNumSamples() == 0)
		{
			UE_LOGF(LogAudio, Error, "Writing out wav file failed- There was no audio data to write.");
			CurrentState = ESoundWavePCMWriterState::Failed;
			return;
		}

		SerializedWavData.Reset();
		SerializeWaveFile(SerializedWavData, (const uint8*)CurrentBuffer.GetData(), CurrentBuffer.GetNumSamples() * sizeof(int16), CurrentBuffer.GetNumChannels(), CurrentBuffer.GetSampleRate());

		UE_LOGF(LogAudio, Verbose, "Serializing %d sample file (%d bytes) to sound asset at %ls", CurrentBuffer.GetNumSamples(), SerializedWavData.Num(), *AbsoluteFilePath);

		// TODO: Check to see if we need to call USoundWave::FreeResources here.

		// Emplace wav data in the RawData component of the sound wave.
		FSharedBuffer Buffer = FSharedBuffer::Clone(SerializedWavData.GetData(), SerializedWavData.Num());

#if WITH_EDITORONLY_DATA
		CurrentSoundWave->RawData.UpdatePayload(Buffer);
#else //WITH_EDITORONLY_DATA
		checkNoEntry();
#endif //WITH_EDITORONLY_DATA

		USoundWave* SavedSoundWave = CurrentSoundWave;

		AsyncTask(ENamedThreads::GameThread, [SavedSoundWave]() {
			FAssetRegistryModule::AssetCreated(SavedSoundWave);
			SavedSoundWave->MarkPackageDirty();
		});
		
		CurrentState = ESoundWavePCMWriterState::Suceeded;
	}

	void FSoundWavePCMWriter::SerializeBufferToWavFile()
	{
		CurrentState = ESoundWavePCMWriterState::Generating;

		if (CurrentBuffer.GetNumSamples() == 0)
		{
			UE_LOGF(LogAudio, Error, "Writing out wav file failed- There was no audio data to write.");
			CurrentState = ESoundWavePCMWriterState::Failed;
			return;
		}

		// Clamp buffer to prevent wraparound when serializing:
		CurrentBuffer.Clamp(0.9999f);
		SerializeWaveFile(SerializedWavData, (const uint8*) CurrentBuffer.GetData(), CurrentBuffer.GetNumSamples() * sizeof(int16), CurrentBuffer.GetNumChannels(), CurrentBuffer.GetSampleRate());
		UE_LOGF(LogAudio, Verbose, "Serializing %d sample file (%d bytes) to %ls", CurrentBuffer.GetNumSamples(), SerializedWavData.Num(), *AbsoluteFilePath);
		if (SerializedWavData.Num() == 0)
		{
			UE_LOGF(LogAudio, Error, "Wave serialize operation failed: failiure in SerializeWaveFile");
			CurrentState = ESoundWavePCMWriterState::Failed;
			return;
		}
		else
		{
			CurrentState = ESoundWavePCMWriterState::WritingToDisk;
			IPlatformFile& PlatformFile = FPlatformFileManager::Get().GetPlatformFile();

			IFileHandle* FileHandle = PlatformFile.OpenWrite(*AbsoluteFilePath);
			if (FileHandle)
			{
				int32 NumChunks = FMath::CeilToInt((float)SerializedWavData.Num() / ChunkSize);
				UE_LOGF(LogAudio, Verbose, "Writing wav file in %d chunks...", NumChunks);
				for (int32 ChunkIndex = 0; ChunkIndex < NumChunks; ChunkIndex++)
				{
					int32 BufferIndex = ChunkIndex * ChunkSize;
					uint8* BufferPtr = &(SerializedWavData.GetData()[BufferIndex]);
					
					// Account for leftover buffer part:
					int32 CurrentChunkSize = FMath::Min(ChunkSize, SerializedWavData.Num() - BufferIndex);
					
					if (!FileHandle->Write(BufferPtr, CurrentChunkSize))
					{
						UE_LOGF(LogAudio, Error, "Wave serialize operation failed: failiure in SerializeWaveFile. Chunk: %d Index: %d ChunkSize: %d", ChunkIndex, BufferIndex, CurrentChunkSize);
						SerializedWavData.Reset();
						delete FileHandle;
						CurrentState = ESoundWavePCMWriterState::Failed;
						return;
					}

					// Update progress:
					Progress.Add(CurrentChunkSize);
				}
				

				// IFileHandle's destructor will close the file.
				delete FileHandle;
			}
			else
			{
				UE_LOGF(LogAudio, Error, "Wave serialize operation failed: failiure in SerializeWaveFile");
				SerializedWavData.Reset();
				CurrentState = ESoundWavePCMWriterState::Failed;
				return;
			}
		}
		UE_LOGF(LogAudio, Verbose, "Succeeded in writing wav file.");
		CurrentState = ESoundWavePCMWriterState::Suceeded;
	}

	bool FSoundWavePCMWriter::CreateDirectoryIfNeeded(FString& DirectoryPath)
	{
		IPlatformFile& PlatformFile = FPlatformFileManager::Get().GetPlatformFile();

		if (!PlatformFile.DirectoryExists(*DirectoryPath))
		{
			if (!PlatformFile.CreateDirectoryTree(*DirectoryPath))
			{
				return false;
			}
		}
		return true;
	}

	FAsyncSoundWavePCMWriteWorker::FAsyncSoundWavePCMWriteWorker(FSoundWavePCMWriter* InWriter, ESoundWavePCMWriteTaskType InTaskType, TFunction<void(const USoundWave*)> OnSuccess)
		: Writer(InWriter)
		, TaskType(InTaskType)
		, CallbackOnSuccess(OnSuccess)
	{
	}

	FAsyncSoundWavePCMWriteWorker::~FAsyncSoundWavePCMWriteWorker()
	{
		FScopeLock AbandonLock(&NonAbandonableSection);
		Abandon();
	}

	void FAsyncSoundWavePCMWriteWorker::DoWork()
	{
	    check (Writer);
		
		switch (TaskType)
		{
			case Audio::ESoundWavePCMWriteTaskType::GenerateSoundWave:
			{
				if (Writer->CurrentSoundWave == nullptr)
				{
					Writer->CurrentState = ESoundWavePCMWriterState::Failed;
					return;
				}
			
				FScopeLock AbandonLock(&NonAbandonableSection);
				Writer->CurrentState = ESoundWavePCMWriterState::Generating;
				Writer->ApplyBufferToSoundWave();
				Writer->CurrentState = ESoundWavePCMWriterState::Suceeded;
				break;
			}

			case Audio::ESoundWavePCMWriteTaskType::GenerateAndWriteSoundWave:
			{
				if (Writer->CurrentSoundWave == nullptr)
				{
					Writer->CurrentState = ESoundWavePCMWriterState::Failed;
					return;
				}

				{
					FScopeLock AbandonLock(&NonAbandonableSection);
					Writer->CurrentState = ESoundWavePCMWriterState::Generating;
					Writer->ApplyBufferToSoundWave();
				}

				{
					FScopeLock AbandonLock(&NonAbandonableSection);
					Writer->CurrentState = ESoundWavePCMWriterState::WritingToDisk;
					Writer->SerializeSoundWaveToAsset();
				}

				break;
			}
			case Audio::ESoundWavePCMWriteTaskType::WriteSoundWave:
			{
				if (Writer->CurrentSoundWave == nullptr)
				{
					Writer->CurrentState = ESoundWavePCMWriterState::Failed;
					return;
				}

				FScopeLock AbandonLock(&NonAbandonableSection);
				Writer->CurrentState = ESoundWavePCMWriterState::WritingToDisk;
				Writer->SerializeSoundWaveToAsset();
				break;
			}

			case Audio::ESoundWavePCMWriteTaskType::WriteWavFile:
			{
				FScopeLock AbandonLock(&NonAbandonableSection);
				Writer->CurrentState = ESoundWavePCMWriterState::WritingToDisk;
				Writer->SerializeBufferToWavFile();
				break;
			}

			default:
				break;
			}

		check (Writer);
		if (Writer->CurrentSoundWave)
		{
			// force update the compression type
			if (IsInAudioThread())
			{
				Writer->CurrentSoundWave->SetSoundAssetCompressionType(ESoundAssetCompressionType::PCM);
			}
			
			// clear out RawPCMData
			if (Writer->CurrentSoundWave->RawPCMData)
			{
				FMemory::Free(Writer->CurrentSoundWave->RawPCMData);
				Writer->CurrentSoundWave->RawPCMData = nullptr;
			}
		}

		// Capture our callback and perform it on the game thread:
		const USoundWave* SoundWave = Writer->CurrentSoundWave;
		TFunction<void(const USoundWave*)> Callback = MoveTemp(CallbackOnSuccess);

		AsyncTask(ENamedThreads::GameThread, [Callback, SoundWave]() {
			Callback(SoundWave);
		});
	}

	void FAsyncSoundWavePCMWriteWorker::Abandon()
	{
		FScopeLock AbandonLock(&NonAbandonableSection);

		// Do any state cleanup here.
		Writer->CurrentState = ESoundWavePCMWriterState::Cancelled;
	}

}

