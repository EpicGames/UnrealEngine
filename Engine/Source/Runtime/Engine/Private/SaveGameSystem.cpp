// Copyright Epic Games, Inc. All Rights Reserved.

#include "SaveGameSystem.h"
#include "HAL/ThreadHeartBeat.h"
#include "Containers/Ticker.h"
#include "Tasks/Pipe.h"


UE::Tasks::FPipe ISaveGameSystem::AsyncTaskPipe{ TEXT("SaveGamePipe") };



void ISaveGameSystem::DoesSaveGameExistAsync(const TCHAR* Name, FPlatformUserId PlatformUserId, FSaveGameAsyncExistsCallback Callback)
{
	FString SlotName(Name);

	// start the save operation on a background thread.
	AsyncTaskPipe.Launch(UE_SOURCE_LOCATION,
		[this, SlotName, PlatformUserId, Callback]()
		{
			// check if the savegame exists
			int32 UserIndex = FPlatformMisc::GetUserIndexForPlatformUser(PlatformUserId);
			const ESaveExistsResult Result = DoesSaveGameExistWithResult(*SlotName, UserIndex);

			// trigger the callback on the game thread.
			if (Callback)
			{
				OnAsyncComplete(
					[SlotName, PlatformUserId, Result, Callback]()
					{
						Callback(SlotName, PlatformUserId, Result);
					}
				);
			}
		}
	);
}


void ISaveGameSystem::SaveGameAsync(bool bAttemptToUseUI, const TCHAR* Name, FPlatformUserId PlatformUserId, TSharedRef<const TArray<uint8>> Data, FSaveGameAsyncOpCompleteCallback Callback)
{
	FString SlotName(Name);

	// start the save operation on a background thread.
	AsyncTaskPipe.Launch(UE_SOURCE_LOCATION,
		[this, bAttemptToUseUI, SlotName, PlatformUserId, Data, Callback]()
		{
			// save the savegame
			int32 UserIndex = FPlatformMisc::GetUserIndexForPlatformUser(PlatformUserId);
			const bool bResult = SaveGame(bAttemptToUseUI, *SlotName, UserIndex, *Data);

			// trigger the callback on the game thread
			if (Callback)
			{
				OnAsyncComplete(
					[SlotName, PlatformUserId, bResult, Callback]()
					{
						Callback(SlotName, PlatformUserId, bResult);
					}
				);
			}
		},
		UE::Tasks::ETaskPriority::Default,
		UE::Tasks::EExtendedTaskPriority::None,
		UE::Tasks::ETaskFlags::DoNotRunInsideBusyWait // it's a long running task
	);
}


void ISaveGameSystem::LoadGameAsync(bool bAttemptToUseUI, const TCHAR* Name, FPlatformUserId PlatformUserId, FSaveGameAsyncLoadCompleteCallback Callback)
{
	FString SlotName(Name);

	// start the load operation on a background thread.
	AsyncTaskPipe.Launch(UE_SOURCE_LOCATION,
		[this, bAttemptToUseUI, SlotName, PlatformUserId, Callback]()
		{
			// load the savegame
			TSharedRef<TArray<uint8>> Data = MakeShared<TArray<uint8>>();
			int32 UserIndex = FPlatformMisc::GetUserIndexForPlatformUser(PlatformUserId);
			const bool bResult = LoadGame(bAttemptToUseUI, *SlotName, UserIndex, Data.Get());

			// trigger the callback on the game thread
			if (Callback)
			{
				OnAsyncComplete(
					[SlotName, PlatformUserId, bResult, Callback, Data]()
					{
						Callback(SlotName, PlatformUserId, bResult, Data.Get());
					}
				);
			}
		}
	);
}

void ISaveGameSystem::LoadGameIfExistsAsync(bool bAttemptToUseUI, const TCHAR* Name, FPlatformUserId PlatformUserId, FSaveGameAsyncLoadCompleteCallback Callback)
{
	FString SlotName(Name);

	// start the load operation on a background thread.
	AsyncTaskPipe.Launch(UE_SOURCE_LOCATION,
		[this, bAttemptToUseUI, SlotName, PlatformUserId, Callback]()
		{
			// load the savegame if it exists
			TSharedRef<TArray<uint8>> Data = MakeShared<TArray<uint8>>();
			int32 UserIndex = FPlatformMisc::GetUserIndexForPlatformUser(PlatformUserId);

			const ESaveExistsResult ExistsResult = DoesSaveGameExistWithResult(*SlotName, UserIndex);
			bool bResult = false;

			if (ExistsResult == ESaveExistsResult::OK)
			{
				bResult = LoadGame(bAttemptToUseUI, *SlotName, UserIndex, Data.Get());
			}

			// trigger the callback on the game thread
			if (Callback)
			{
				OnAsyncComplete(
					[SlotName, PlatformUserId, bResult, Callback, Data]()
					{
						Callback(SlotName, PlatformUserId, bResult, Data.Get());
					}
				);
			}
		}
	);
}


void ISaveGameSystem::DeleteGameAsync(bool bAttemptToUseUI, const TCHAR* Name, FPlatformUserId PlatformUserId, FSaveGameAsyncOpCompleteCallback Callback)
{
	FString SlotName(Name);

	// start the delete operation on a background thread.
	AsyncTaskPipe.Launch(UE_SOURCE_LOCATION,
		[this, bAttemptToUseUI, SlotName, PlatformUserId, Callback]()
		{
			// delete the savegame
			int32 UserIndex = FPlatformMisc::GetUserIndexForPlatformUser(PlatformUserId);
			const bool bResult = DeleteGame(bAttemptToUseUI, *SlotName, UserIndex);

			// trigger the callback on the game thread
			if (Callback)
			{
				OnAsyncComplete(
					[SlotName, PlatformUserId, bResult, Callback]()
					{
						Callback(SlotName, PlatformUserId, bResult);
					}
				);
			}
		}
	);
}

void ISaveGameSystem::GetSaveGameNamesAsync(FPlatformUserId PlatformUserId, FSaveGameAsyncGetNamesCallback Callback)
{
	// start the delete operation on a background thread.
	AsyncTaskPipe.Launch(UE_SOURCE_LOCATION,
		[this, PlatformUserId, Callback]()
		{
			// get the list of savegames
			TArray<FString> FoundSaves;
			int32 UserIndex = FPlatformMisc::GetUserIndexForPlatformUser(PlatformUserId);
			const bool bResult = GetSaveGameNames(FoundSaves, UserIndex);

			// trigger the callback on the game thread
			if (Callback)
			{
				OnAsyncComplete(
					[PlatformUserId, bResult, FoundSaves = MoveTemp(FoundSaves), Callback]()
					{
						Callback(PlatformUserId, bResult, FoundSaves);
					}
				);
			}
		}
	);

}


void ISaveGameSystem::InitAsync(bool bAttemptToUseUI, FPlatformUserId PlatformUserId, FSaveGameAsyncInitCompleteCallback Callback)
{
	// default implementation does nothing, so just trigger the completion callback on the game thread immediately
	if (Callback)
	{
		OnAsyncComplete(
			[PlatformUserId, Callback]()
			{
				Callback(PlatformUserId, true);
			}
		);
	}
}


void ISaveGameSystem::OnAsyncComplete(TFunction<void()> Callback)
{
	// NB. Using Ticker because AsyncTask may run during async package loading which may not be suitable for save data
	FTSTicker::GetCoreTicker().AddTicker(FTickerDelegate::CreateLambda(
		[Callback = MoveTemp(Callback)](float) -> bool
		{
			Callback();
			return false;
		}
	));
}








ISaveGameSystem::ESaveExistsResult FBaseAsyncSaveGameSystem::DoesSaveGameExistWithResult(const TCHAR* Name, const int32 UserIndex)
{
	TSharedPtr<ESaveExistsResult> Result = MakeShared<ESaveExistsResult>(ESaveExistsResult::UnspecifiedError);

	const FPlatformUserId PlatformUserId = FPlatformMisc::GetPlatformUserForUserIndex(UserIndex);

	UE::Tasks::FTask Op = InternalDoesSaveGameExistAsync(Name, PlatformUserId, nullptr, Result);
	WaitForAsyncTask(Op);

	return *Result;
}

bool FBaseAsyncSaveGameSystem::SaveGame(bool bAttemptToUseUI, const TCHAR* Name, const int32 UserIndex, const TArray<uint8>& Data)
{
	TSharedPtr<bool> Result = MakeShared<bool>(false);
	TSharedRef<const TArray<uint8>> DataPtr = MakeShared<const TArray<uint8>>(Data); // have to take a copy

	const FPlatformUserId PlatformUserId = FPlatformMisc::GetPlatformUserForUserIndex(UserIndex);
	UE::Tasks::FTask Op = InternalSaveGameAsync(bAttemptToUseUI, Name, PlatformUserId, DataPtr, nullptr, Result);
	WaitForAsyncTask(Op);

	return *Result;
}

bool FBaseAsyncSaveGameSystem::LoadGame(bool bAttemptToUseUI, const TCHAR* Name, const int32 UserIndex, TArray<uint8>& Data)
{
	TSharedPtr<bool> Result = MakeShared<bool>(false);
	TSharedRef<TArray<uint8>> DataPtr = MakeShared<TArray<uint8>>(Data); // have to take a copy

	const FPlatformUserId PlatformUserId = FPlatformMisc::GetPlatformUserForUserIndex(UserIndex);
	UE::Tasks::FTask Op = InternalLoadGameAsync(bAttemptToUseUI, Name, PlatformUserId, DataPtr, nullptr, Result);
	WaitForAsyncTask(Op);

	Data = MoveTemp(*DataPtr);
	return  *Result;
}

bool FBaseAsyncSaveGameSystem::DeleteGame(bool bAttemptToUseUI, const TCHAR* Name, const int32 UserIndex)
{
	TSharedPtr<bool> Result = MakeShared<bool>(false);

	const FPlatformUserId PlatformUserId = FPlatformMisc::GetPlatformUserForUserIndex(UserIndex);
	UE::Tasks::FTask Op = InternalDeleteGameAsync(bAttemptToUseUI, Name, PlatformUserId, nullptr, Result);
	WaitForAsyncTask(Op);

	return *Result;
}

bool FBaseAsyncSaveGameSystem::GetSaveGameNames(TArray<FString>& FoundSaves, const int32 UserIndex)
{
	TSharedPtr<bool> Result = MakeShared<bool>(false);
	TSharedRef<TArray<FString>> FoundSavesPtr = MakeShared<TArray<FString>>();

	const FPlatformUserId PlatformUserId = FPlatformMisc::GetPlatformUserForUserIndex(UserIndex);
	UE::Tasks::FTask Op = InternalGetSaveGameNamesAsync(PlatformUserId, FoundSavesPtr, nullptr, Result);
	WaitForAsyncTask(Op);

	FoundSaves = MoveTemp(*FoundSavesPtr);
	return *Result;
}



void FBaseAsyncSaveGameSystem::DoesSaveGameExistAsync(const TCHAR* Name, FPlatformUserId PlatformUserId, FSaveGameAsyncExistsCallback Callback)
{
	InternalDoesSaveGameExistAsync(Name, PlatformUserId, Callback);
}

void FBaseAsyncSaveGameSystem::SaveGameAsync(bool bAttemptToUseUI, const TCHAR* Name, FPlatformUserId PlatformUserId, TSharedRef<const TArray<uint8>> Data, FSaveGameAsyncOpCompleteCallback Callback)
{
	InternalSaveGameAsync(bAttemptToUseUI, Name, PlatformUserId, Data, Callback);
}

void FBaseAsyncSaveGameSystem::LoadGameAsync(bool bAttemptToUseUI, const TCHAR* Name, FPlatformUserId PlatformUserId, FSaveGameAsyncLoadCompleteCallback Callback)
{
	TSharedRef<TArray<uint8>> Data = MakeShared<TArray<uint8>>();
	InternalLoadGameAsync(bAttemptToUseUI, Name, PlatformUserId, Data, Callback);
}

void FBaseAsyncSaveGameSystem::LoadGameIfExistsAsync(bool bAttemptToUseUI, const TCHAR* Name, FPlatformUserId PlatformUserId, FSaveGameAsyncLoadCompleteCallback Callback)
{
	// If a platform has a custom async implementation it is not safe to use the generic task pipe. So, fall back to calling the normal async load
	// Platforms can override this if InternalDoesSaveGameExistAsync always needs to be called before InternalLoadGameAsync
	LoadGameAsync(bAttemptToUseUI, Name, PlatformUserId, Callback);
}

void FBaseAsyncSaveGameSystem::DeleteGameAsync(bool bAttemptToUseUI, const TCHAR* Name, FPlatformUserId PlatformUserId, FSaveGameAsyncOpCompleteCallback Callback)
{
	InternalDeleteGameAsync(bAttemptToUseUI, Name, PlatformUserId, Callback);
}

void FBaseAsyncSaveGameSystem::GetSaveGameNamesAsync(FPlatformUserId PlatformUserId, FSaveGameAsyncGetNamesCallback Callback)
{
	TSharedRef<TArray<FString>> FoundSavesPtr = MakeShared<TArray<FString>>();
	InternalGetSaveGameNamesAsync(PlatformUserId, FoundSavesPtr, Callback);
}




void FBaseAsyncSaveGameSystem::WaitForAsyncTask(UE::Tasks::FTask AsyncSaveTask)
{
	// need to pump messages on the game thread
	if (IsInGameThread())
	{
		// Suspend the hang and hitch heartbeats, as this is a long running task.
		FSlowHeartBeatScope SuspendHeartBeat;
		FDisableHitchDetectorScope SuspendGameThreadHitch;

		while (!AsyncSaveTask.IsCompleted())
		{
			FPlatformMisc::PumpMessagesOutsideMainLoop();
		}
	}
	else
	{
		// not running on the game thread, so just block until the async operation comes back
		AsyncSaveTask.Wait();
	}
}
