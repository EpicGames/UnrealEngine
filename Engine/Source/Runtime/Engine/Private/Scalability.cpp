// Copyright Epic Games, Inc. All Rights Reserved.

#include "Scalability.h"
#include "GenericPlatform/GenericPlatformSurvey.h"
#include "IAnalyticsProviderET.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/ConfigUtilities.h"
#include "HAL/IConsoleManager.h"
#include "RHIShaderPlatform.h"
#include "SynthBenchmark.h"
#include "EngineAnalytics.h"
#include "Interfaces/IProjectManager.h"

#if WITH_EDITOR
#include "Editor.h"
#include "DeviceProfiles/DeviceProfile.h"
#include "DeviceProfiles/DeviceProfileManager.h"
#endif

Scalability::FOnScalabilitySettingsChanged Scalability::OnScalabilitySettingsChanged;

#if !UE_BUILD_SHIPPING

static TAutoConsoleVariable<float> CVarTestCPUPerfIndexOverride(
	TEXT("sg.Test.CPUPerfIndexOverride"), 0.0f,
	TEXT("Custom override for the CPU perf index returned by the GPU benchmark."),
	ECVF_Default);

static TAutoConsoleVariable<float> CVarTestGPUPerfIndexOverride(
	TEXT("sg.Test.GPUPerfIndexOverride"), 0.0f,
	TEXT("Custom override for the GPU perf index returned by the GPU benchmark."),
	ECVF_Default);

#endif

static TAutoConsoleVariable<float> CVarResolutionQuality(
	TEXT("sg.ResolutionQuality"),
	0.0f,
	TEXT("Scalability quality state (internally used by scalability system, ini load/save or using SCALABILITY console command)\n")
	TEXT(" 10..100, default: 100"),
	ECVF_Scalability);

static TAutoConsoleVariable<int32> CVarViewDistanceQuality(
	TEXT("sg.ViewDistanceQuality"),
	Scalability::DefaultQualityLevel,
	TEXT("Scalability quality state (internally used by scalability system, ini load/save or using SCALABILITY console command)\n")
	TEXT(" 0:low, 1:med, 2:high, 3:epic, 4:cinematic, default: 3"),
	ECVF_ScalabilityGroup | ECVF_Preview);

static TAutoConsoleVariable<int32> CVarAntiAliasingQuality(
	TEXT("sg.AntiAliasingQuality"),
	Scalability::DefaultQualityLevel,
	TEXT("Scalability quality state (internally used by scalability system, ini load/save or using SCALABILITY console command)\n")
	TEXT(" 0:low, 1:med, 2:high, 3:epic, 4:cinematic, default: 3"),
	ECVF_ScalabilityGroup | ECVF_Preview);

static TAutoConsoleVariable<int32> CVarShadowQuality(
	TEXT("sg.ShadowQuality"),
	Scalability::DefaultQualityLevel,
	TEXT("Scalability quality state (internally used by scalability system, ini load/save or using SCALABILITY console command)\n")
	TEXT(" 0:low, 1:med, 2:high, 3:epic, 4:cinematic, default: 3"),
	ECVF_ScalabilityGroup | ECVF_Preview);

static TAutoConsoleVariable<int32> CVarGlobalIlluminationQuality(
	TEXT("sg.GlobalIlluminationQuality"),
	Scalability::DefaultQualityLevel,
	TEXT("Scalability quality state (internally used by scalability system, ini load/save or using SCALABILITY console command)\n")
	TEXT(" 0:low, 1:med, 2:high, 3:epic, 4:cinematic, default: 3"),
	ECVF_ScalabilityGroup | ECVF_Preview);

static TAutoConsoleVariable<int32> CVarReflectionQuality(
	TEXT("sg.ReflectionQuality"),
	Scalability::DefaultQualityLevel,
	TEXT("Scalability quality state (internally used by scalability system, ini load/save or using SCALABILITY console command)\n")
	TEXT(" 0:low, 1:med, 2:high, 3:epic, 4:cinematic, default: 3"),
	ECVF_ScalabilityGroup | ECVF_Preview);

static TAutoConsoleVariable<int32> CVarPostProcessQuality(
	TEXT("sg.PostProcessQuality"),
	Scalability::DefaultQualityLevel,
	TEXT("Scalability quality state (internally used by scalability system, ini load/save or using SCALABILITY console command)\n")
	TEXT(" 0:low, 1:med, 2:high, 3:epic, 4:cinematic, default: 3"),
	ECVF_ScalabilityGroup | ECVF_Preview);

static TAutoConsoleVariable<int32> CVarTextureQuality(
	TEXT("sg.TextureQuality"),
	Scalability::DefaultQualityLevel,
	TEXT("Scalability quality state (internally used by scalability system, ini load/save or using SCALABILITY console command)\n")
	TEXT(" 0:low, 1:med, 2:high, 3:epic, 4:cinematic, default: 3"),
	ECVF_ScalabilityGroup | ECVF_Preview);

static TAutoConsoleVariable<int32> CVarEffectsQuality(
	TEXT("sg.EffectsQuality"),
	Scalability::DefaultQualityLevel,
	TEXT("Scalability quality state (internally used by scalability system, ini load/save or using SCALABILITY console command)\n")
	TEXT(" 0:low, 1:med, 2:high, 3:epic, 4:cinematic, default: 3"),
	ECVF_ScalabilityGroup | ECVF_Preview);

static TAutoConsoleVariable<int32> CVarFoliageQuality(
	TEXT("sg.FoliageQuality"),
	3,
	TEXT("Scalability quality state (internally used by scalability system, ini load/save or using SCALABILITY console command)\n")
	TEXT(" 0:low, 1:med, 2:high, 3:epic, 4:cinematic, default: 3"),
	ECVF_ScalabilityGroup | ECVF_Preview);

static TAutoConsoleVariable<int32> CVarShadingQuality(
	TEXT("sg.ShadingQuality"),
	Scalability::DefaultQualityLevel,
	TEXT("Scalability quality state (internally used by scalability system, ini load/save or using SCALABILITY console command)\n")
	TEXT(" 0:low, 1:med, 2:high, 3:epic, 4:cinematic, default: 3"),
	ECVF_ScalabilityGroup | ECVF_Preview);

static TAutoConsoleVariable<int32> CVarLandscapeQuality(
	TEXT("sg.LandscapeQuality"),
	Scalability::DefaultQualityLevel,
	TEXT("Scalability quality state (internally used by scalability system, ini load/save or using SCALABILITY console command)\n")
	TEXT(" 0:low, 1:med, 2:high, 3:epic, 4:cinematic, default: 3"),
	ECVF_ScalabilityGroup | ECVF_Preview);

static TAutoConsoleVariable<int32> CVarViewDistanceQuality_NumLevels(
	TEXT("sg.ViewDistanceQuality.NumLevels"),
	5,
	TEXT("Number of settings quality levels in sg.ViewDistanceQuality\n")
	TEXT(" default: 5 (0..4)"),
	ECVF_ReadOnly);

static TAutoConsoleVariable<int32> CVarAntiAliasingQuality_NumLevels(
	TEXT("sg.AntiAliasingQuality.NumLevels"),
	5,
	TEXT("Number of settings quality levels in sg.AntiAliasingQuality\n")
	TEXT(" default: 5 (0..4)"),
	ECVF_ReadOnly);

static TAutoConsoleVariable<int32> CVarShadowQuality_NumLevels(
	TEXT("sg.ShadowQuality.NumLevels"),
	5,
	TEXT("Number of settings quality levels in sg.ShadowQuality\n")
	TEXT(" default: 5 (0..4)"),
	ECVF_ReadOnly);

static TAutoConsoleVariable<int32> CVarGlobalIlluminationQuality_NumLevels(
	TEXT("sg.GlobalIlluminationQuality.NumLevels"),
	5,
	TEXT("Number of settings quality levels in sg.GlobalIlluminationQuality\n")
	TEXT(" default: 5 (0..4)"),
	ECVF_ReadOnly);

static TAutoConsoleVariable<int32> CVarReflectionQuality_NumLevels(
	TEXT("sg.ReflectionQuality.NumLevels"),
	5,
	TEXT("Number of settings quality levels in sg.ReflectionQuality\n")
	TEXT(" default: 5 (0..4)"),
	ECVF_ReadOnly);

static TAutoConsoleVariable<int32> CVarPostProcessQuality_NumLevels(
	TEXT("sg.PostProcessQuality.NumLevels"),
	5,
	TEXT("Number of settings quality levels in sg.PostProcessQuality\n")
	TEXT(" default: 5 (0..4)"),
	ECVF_ReadOnly);

static TAutoConsoleVariable<int32> CVarTextureQuality_NumLevels(
	TEXT("sg.TextureQuality.NumLevels"),
	5,
	TEXT("Number of settings quality levels in sg.TextureQuality\n")
	TEXT(" default: 5 (0..4)"),
	ECVF_ReadOnly);

static TAutoConsoleVariable<int32> CVarEffectsQuality_NumLevels(
	TEXT("sg.EffectsQuality.NumLevels"),
	5,
	TEXT("Number of settings quality levels in sg.EffectsQuality\n")
	TEXT(" default: 5 (0..4)"),
	ECVF_ReadOnly);
	
static TAutoConsoleVariable<int32> CVarFoliageQuality_NumLevels(
	TEXT("sg.FoliageQuality.NumLevels"),
	5,
	TEXT("Number of settings quality levels in sg.FoliageQuality\n")
	TEXT(" default: 5 (0..4)"),
	ECVF_ReadOnly);

static TAutoConsoleVariable<int32> CVarShadingQuality_NumLevels(
	TEXT("sg.ShadingQuality.NumLevels"),
	5,
	TEXT("Number of settings quality levels in sg.ShadingQuality\n")
	TEXT(" default: 5 (0..4)"),
	ECVF_ReadOnly);

static TAutoConsoleVariable<int32> CVarLandscapeQuality_NumLevels(
	TEXT("sg.LandscapeQuality.NumLevels"),
	5,
	TEXT("Number of settings quality levels in sg.LandscapeQuality\n")
	TEXT(" default: 5 (0..4)"),
	ECVF_ReadOnly);

namespace Scalability
{
static FQualityLevels GScalabilityBackupQualityLevels(false);
static FQualityLevels GScalabilityQualityLevelsOverride(false);
static bool GScalabilityUsingTemporaryQualityLevels = false;
static bool bIsInitialized = false;

// Select a the correct quality level for the given benchmark value and thresholds
int32 ComputeOptionFromPerfIndex(const FString& GroupName, float CPUPerfIndex, float GPUPerfIndex)
{
	// Some code defaults in case the ini file can not be read or has dirty data
	float PerfIndex = FMath::Min(CPUPerfIndex, GPUPerfIndex);

	TArray<float, TInlineAllocator<4>> Thresholds;
	Thresholds.Add(20.0f);
	Thresholds.Add(50.0f);
	Thresholds.Add(70.0f);

	if (GConfig)
	{
		const FString ArrayKey = FString("PerfIndexThresholds_") + GroupName;
		TArray<FString> PerfIndexThresholds;
		GConfig->GetSingleLineArray(TEXT("ScalabilitySettings"), *ArrayKey, PerfIndexThresholds, GScalabilityIni);

		// This array takes on the form: "TypeString Index01 Index12 Index23"
		if (PerfIndexThresholds.Num() > 1)
		{
			const FString TypeString = PerfIndexThresholds[0];
			bool bTypeValid = false;
			if (TypeString == TEXT("CPU"))
			{
				PerfIndex = CPUPerfIndex;
				bTypeValid = true;
			}
			else if (TypeString == TEXT("GPU"))
			{
				PerfIndex = GPUPerfIndex;
				bTypeValid = true;
			}
			else if (TypeString == TEXT("Min"))
			{
				PerfIndex = FMath::Min(CPUPerfIndex, GPUPerfIndex);
				bTypeValid = true;
			}

			if (bTypeValid)
			{
				Thresholds.Reset();
				for (int32 ParseIndex = 1; ParseIndex < PerfIndexThresholds.Num(); ++ParseIndex)
				{
					const float Threshold = FCString::Atof(*PerfIndexThresholds[ParseIndex]);
					Thresholds.Add(Threshold);
				}
			}
		}
	}

	// Threshold the value
	int32 ResultIndex = 0;
	for (float Threshold : Thresholds)
	{
		if (PerfIndex < Threshold)
		{
			break;
		}
		++ResultIndex;
	}
	return ResultIndex;
}

// Extract the name and quality level from an ini section name. Sections in the ini file are named <GroupName>@<QualityLevel> 
static bool SplitSectionName(const FString& InSectionName, FString& OutSectionGroupName, int32& OutQualityLevel)
{
	bool bSuccess = false;
	FString Left, Right;

	if (InSectionName.Split(TEXT("@"), &Left, &Right))
	{
		OutSectionGroupName = Left;
		OutQualityLevel = FCString::Atoi(*Right);
		bSuccess = true;
	}

	return bSuccess;
}

// Try and match the current cvar state against the scalability sections too see if one matches. OutQualityLevel will be set to -1 if none match
static void InferCurrentQualityLevel(const FString& InGroupName, int32& OutQualityLevel, TArray<FString>* OutCVars)
{
	TArray<FString> SectionNames;
	GConfig->GetSectionNames(GScalabilityIni, SectionNames);
	OutQualityLevel = -1;

	for (int32 SectionNameIndex = 0; SectionNameIndex < SectionNames.Num(); ++SectionNameIndex)
	{
		FString GroupName;
		int32 GroupQualityLevel;

		if (SplitSectionName(SectionNames[SectionNameIndex], GroupName, GroupQualityLevel))
		{
			if (GroupName == FString(InGroupName))
			{
				TArray<FString> CVarData;
				GConfig->GetSection(*SectionNames[SectionNameIndex], CVarData, GScalabilityIni);

				bool bAllMatch = true;

				// Check all cvars against current state to see if they match
				for (int32 DataIndex = 0; DataIndex < CVarData.Num(); ++DataIndex)
				{
					const FString& CVarString = CVarData[DataIndex];
					FString CVarName, CVarValue;
					if (CVarString.Split(TEXT("="), &CVarName, &CVarValue))
					{
						IConsoleVariable* CVar = IConsoleManager::Get().FindConsoleVariable(*CVarName);
						if (CVar)
						{
							float CurrentValue = CVar->GetFloat();
							float ValueToCheck = FCString::Atof(*CVarValue);

							if (ValueToCheck != CurrentValue)
							{
								bAllMatch = false;
								break;
							}
						}
					}
				}

				if (bAllMatch)
				{
					OutQualityLevel = FMath::Max(OutQualityLevel, GroupQualityLevel);
					if (OutCVars)
					{
						*OutCVars = CVarData;
					}
				}
			}
		}
	}
}


FString GetScalabilitySectionString(const TCHAR* InGroupName, int32 InQualityLevel, int32 InNumLevels)
{
	check(InNumLevels > 0);
	const int32 MaxLevel = InNumLevels - 1;
	InQualityLevel = FMath::Clamp(InQualityLevel, 0, MaxLevel);
	FString Result;

	if (InQualityLevel == MaxLevel)
	{
		Result = FString::Printf(TEXT("%s@Cine"), InGroupName);
	}
	else
	{
		Result = FString::Printf(TEXT("%s@%d"), InGroupName, InQualityLevel);
	}

	return Result;
}

#if WITH_EDITOR
FName PlatformScalabilityName; // The name of the current platform scalability, or NAME_None if none is active
EShaderPlatform ScalabilityShaderPlatform;
bool bScalabilityShaderPlatformHasBeenChanged;
FString PlatformScalabilityIniFilename;
TMap<IConsoleVariable*, FString> PlatformScalabilityCVarBackup;
TSet<const IConsoleVariable*> PlatformScalabilityCVarAllowList;
TSet<const IConsoleVariable*> PlatformScalabilityCVarDenyList;
TMap<TTuple<EShaderPlatform, FName>, Scalability::FQualityLevels> PreviewShaderPlatformToCachedQualityLevels;
TMap<TTuple<EShaderPlatform, FName>, Scalability::FQualityLevels> DefaultPreviewShaderPlatformToCachedQualityLevels;

void UndoPlatformScalability()
{
	// Reapply all CVars to the previous values  
	for (auto It = PlatformScalabilityCVarBackup.CreateIterator(); It; ++It)
	{
		It.Key()->Set(*It.Value(), ECVF_SetByScalability);
	}
	PlatformScalabilityCVarBackup.Empty();
}

void ApplyScalabilityGroupFromPlatformIni(const TCHAR* InSectionName, const TCHAR* InIniFilename)
{
	UE_LOGF(LogConfig, Log, "Applying CVar settings from Section [%ls] File [%ls]", InSectionName, InIniFilename);

	TFunction<void(IConsoleVariable*, const FString&, const FString&)> Func = [&](IConsoleVariable* CVar, const FString& KeyString, const FString& ValueString)
	{
		// Check the allow/deny lists
		if (!PlatformScalabilityCVarDenyList.Contains(CVar) && (PlatformScalabilityCVarAllowList.Num() == 0 || PlatformScalabilityCVarAllowList.Contains(CVar)))
		{
			// Backup cvar we're going to overwrite with the platform specific
			if (!PlatformScalabilityCVarBackup.Contains(CVar))
			{
				PlatformScalabilityCVarBackup.Add(CVar, CVar->GetString());
			}

			// Apply the platform override
			UE_LOGF(LogConfig, Verbose, "Setting scalability CVar [[%ls:%ls]]", *KeyString, *ValueString);
			CVar->Set(*ValueString, ECVF_SetByScalability);
		}			   
	};
			
	UE::ConfigUtilities::ForEachCVarInSectionFromIni(InSectionName, InIniFilename, Func);
}

void ChangeScalabilityPreviewPlatform(FName NewPlatformScalabilityName, const EShaderPlatform& ShaderPlatform)
{
	PlatformScalabilityName = NewPlatformScalabilityName;
	bScalabilityShaderPlatformHasBeenChanged = ScalabilityShaderPlatform != ShaderPlatform;
	ScalabilityShaderPlatform = ShaderPlatform;
}
#endif

static void SetGroupQualityLevel(const TCHAR* InGroupName, int32 InQualityLevel, int32 InNumLevels)
{
//	UE_LOGF(LogConsoleResponse, Display, "  %ls %d", InGroupName, InQualityLevel);
	FString Section = GetScalabilitySectionString(InGroupName, InQualityLevel, InNumLevels);

#if WITH_EDITOR
	if (PlatformScalabilityName != NAME_None)
	{
		if (FConfigCacheIni* PlatformIni = FConfigCacheIni::ForPlatform(PlatformScalabilityName))
		{
			TArray<FString> CVarData;
			PlatformIni->GetSection(*Section, CVarData, GScalabilityIni);
			{
				// Check all cvars against current state to see if they match
				for (int32 DataIndex = 0; DataIndex < CVarData.Num(); ++DataIndex)
				{
					const FString& CVarString = CVarData[DataIndex];
					FString CVarName, CVarValue;
					if (CVarString.Split(TEXT("="), &CVarName, &CVarValue))
					{
						if (IConsoleVariable* CVar = IConsoleManager::Get().FindConsoleVariable(*CVarName))
						{
							EConsoleVariableFlags MaxPriority = ECVF_SetByPreview;
							EConsoleVariableFlags MinPriority = ECVF_SetByScalability;
							CVar->ReplaceCurrentPriorityAndTag(*CVarValue, MaxPriority, MinPriority);
						}
					}
				}
			}
		}
	}
	else
#endif
	{
		UE::ConfigUtilities::ApplyCVarSettingsFromIni(*Section, *GScalabilityIni, ECVF_SetByScalability);
	}
}

#if WITH_EDITOR
void ApplyCachedQualityLevels()
{
	// We directly use UDeviceProfileManager::DeviceProfileManagerSingleton here so we don't unexpectedly initialize the device profile system during cooking.
	if (ScalabilityShaderPlatform != EShaderPlatform::SP_NumPlatforms && UDeviceProfileManager::DeviceProfileManagerSingleton)
	{
		const UDeviceProfile* PreviewDeviceProfile = UDeviceProfileManager::DeviceProfileManagerSingleton->GetPreviewDeviceProfile();
		if (PreviewDeviceProfile)
		{
			TTuple<EShaderPlatform, FName> PreviewShaderPlatformKey = MakeTuple(ScalabilityShaderPlatform, PreviewDeviceProfile->GetFName());

			DefaultPreviewShaderPlatformToCachedQualityLevels.FindOrAdd(PreviewShaderPlatformKey) = Scalability::GetQualityLevels();
			if (Scalability::FQualityLevels* CachedQualityLevel = PreviewShaderPlatformToCachedQualityLevels.Find(PreviewShaderPlatformKey))
			{
				Scalability::SetQualityLevels(*CachedQualityLevel);
				Scalability::SaveState(GEditorSettingsIni);
			}
		}
	}
}

FQualityLevels GetDefaultQualityLevels()
{
	// We directly use UDeviceProfileManager::DeviceProfileManagerSingleton here so we don't unexpectedly initialize the device profile system during cooking.
	if (ScalabilityShaderPlatform != EShaderPlatform::SP_NumPlatforms && UDeviceProfileManager::DeviceProfileManagerSingleton)
	{
		const UDeviceProfile* PreviewDeviceProfile = UDeviceProfileManager::DeviceProfileManagerSingleton->GetPreviewDeviceProfile();
		if (PreviewDeviceProfile)
		{
			if (Scalability::FQualityLevels* DefaultQualityLevels = DefaultPreviewShaderPlatformToCachedQualityLevels.Find(MakeTuple(ScalabilityShaderPlatform, PreviewDeviceProfile->GetFName())))
			{
				return *DefaultQualityLevels;
			}
		}
	}
	
	FQualityLevels QualityLevels;
	QualityLevels.SetFromSingleQualityLevelRelativeToMax(1);
	return QualityLevels;
}

void ResetQualityLevelsToDefault()
{
	SetQualityLevels(GetDefaultQualityLevels());
}
#endif

float GetResolutionScreenPercentage()
{
	static IConsoleVariable* CVar = IConsoleManager::Get().FindConsoleVariable(TEXT("r.ScreenPercentage"));
	return FMath::Clamp(CVar->GetFloat(), MinResolutionScale, MaxResolutionScale);
}

FText GetScalabilityNameFromQualityLevel(int32 QualityLevel)
{
#define LOCTEXT_NAMESPACE "EngineScalabiltySettings"
	static const FText NamesLow(LOCTEXT("QualityLowLabel", "Low"));
	static const FText NamesMedium(LOCTEXT("QualityMediumLabel", "Medium"));
	static const FText NamesHigh(LOCTEXT("QualityHighLabel", "High"));
	static const FText NamesEpic(LOCTEXT("QualityEpicLabel", "Epic"));
	static const FText NamesCine(LOCTEXT("QualityCineLabel", "Cinematic"));

	switch (QualityLevel)
	{
	case 0:
		return NamesLow;
	case 1:
		return NamesMedium;
	case 2:
		return NamesHigh;
	case 3: 
		return NamesEpic;
	case 4:
		return NamesCine;
	default:
		ensureMsgf(false, TEXT("Scalability Level %d needs a display name"), QualityLevel);
		return FText::GetEmpty();
	}

#undef LOCTEXT_NAMESPACE
}

static void SetResolutionQualityLevel(float InResolutionQualityLevel)
{
	if (InResolutionQualityLevel == 0.0)
	{
		// NOP: just use the project's default screen percentage.
		return;
	}
	else
	{
		InResolutionQualityLevel = FMath::Clamp(InResolutionQualityLevel, Scalability::MinResolutionScale, Scalability::MaxResolutionScale);
	}

//	UE_LOGF(LogConsoleResponse, Display, "  ResolutionQuality %.2f", "", InResolutionQualityLevel);

	static IConsoleVariable* CVar = IConsoleManager::Get().FindConsoleVariable(TEXT("r.ScreenPercentage"));

	// if it wasn't created yet we either need to change the order or store like we do for ini loading
	check(CVar);

	CVar->Set(InResolutionQualityLevel, ECVF_SetByScalability);
}

void OnChangeResolutionQuality(IConsoleVariable* Var)
{
	SetResolutionQualityLevel(Var->GetFloat());
}

void OnChangeViewDistanceQuality(IConsoleVariable* Var)
{
	SetGroupQualityLevel(TEXT("ViewDistanceQuality"), Var->GetInt(), CVarViewDistanceQuality_NumLevels->GetInt());
}

void OnChangeAntiAliasingQuality(IConsoleVariable* Var)
{
	SetGroupQualityLevel(TEXT("AntiAliasingQuality"), Var->GetInt(), CVarAntiAliasingQuality_NumLevels->GetInt());
}

void OnChangeShadowQuality(IConsoleVariable* Var)
{
	SetGroupQualityLevel(TEXT("ShadowQuality"), Var->GetInt(), CVarShadowQuality_NumLevels->GetInt());
}

void OnChangeGlobalIlluminationQuality(IConsoleVariable* Var)
{
	SetGroupQualityLevel(TEXT("GlobalIlluminationQuality"), Var->GetInt(), CVarGlobalIlluminationQuality_NumLevels->GetInt());
}

void OnChangeReflectionQuality(IConsoleVariable* Var)
{
	SetGroupQualityLevel(TEXT("ReflectionQuality"), Var->GetInt(), CVarReflectionQuality_NumLevels->GetInt());
}

void OnChangePostProcessQuality(IConsoleVariable* Var)
{
	SetGroupQualityLevel(TEXT("PostProcessQuality"), Var->GetInt(), CVarPostProcessQuality_NumLevels->GetInt());
}

void OnChangeTextureQuality(IConsoleVariable* Var)
{
	SetGroupQualityLevel(TEXT("TextureQuality"), Var->GetInt(), CVarTextureQuality_NumLevels->GetInt());
}

void OnChangeEffectsQuality(IConsoleVariable* Var)
{
	SetGroupQualityLevel(TEXT("EffectsQuality"), Var->GetInt(), CVarEffectsQuality_NumLevels->GetInt());
}

void OnChangeFoliageQuality(IConsoleVariable* Var)
{
	SetGroupQualityLevel(TEXT("FoliageQuality"), Var->GetInt(), CVarFoliageQuality_NumLevels->GetInt());
}

void OnChangeShadingQuality(IConsoleVariable* Var)
{
	SetGroupQualityLevel(TEXT("ShadingQuality"), Var->GetInt(), CVarShadingQuality_NumLevels->GetInt());
}

void OnChangeLandscapeQuality(IConsoleVariable* Var)
{
	SetGroupQualityLevel(TEXT("LandscapeQuality"), Var->GetInt(), CVarLandscapeQuality_NumLevels->GetInt());
}
	
void InitScalabilitySystem()
{
	// needed only once
	if (bIsInitialized)
	{
		return;
	}

	CVarResolutionQuality.AsVariable()->SetOnChangedCallback(FConsoleVariableDelegate::CreateStatic(&OnChangeResolutionQuality));
	CVarViewDistanceQuality.AsVariable()->SetOnChangedCallback(FConsoleVariableDelegate::CreateStatic(&OnChangeViewDistanceQuality));
	CVarAntiAliasingQuality.AsVariable()->SetOnChangedCallback(FConsoleVariableDelegate::CreateStatic(&OnChangeAntiAliasingQuality));
	CVarShadowQuality.AsVariable()->SetOnChangedCallback(FConsoleVariableDelegate::CreateStatic(&OnChangeShadowQuality));
	CVarGlobalIlluminationQuality.AsVariable()->SetOnChangedCallback(FConsoleVariableDelegate::CreateStatic(&OnChangeGlobalIlluminationQuality));
	CVarReflectionQuality.AsVariable()->SetOnChangedCallback(FConsoleVariableDelegate::CreateStatic(&OnChangeReflectionQuality));
	CVarPostProcessQuality.AsVariable()->SetOnChangedCallback(FConsoleVariableDelegate::CreateStatic(&OnChangePostProcessQuality));
	CVarTextureQuality.AsVariable()->SetOnChangedCallback(FConsoleVariableDelegate::CreateStatic(&OnChangeTextureQuality));
	CVarEffectsQuality.AsVariable()->SetOnChangedCallback(FConsoleVariableDelegate::CreateStatic(&OnChangeEffectsQuality));
	CVarFoliageQuality.AsVariable()->SetOnChangedCallback(FConsoleVariableDelegate::CreateStatic(&OnChangeFoliageQuality));
	CVarShadingQuality.AsVariable()->SetOnChangedCallback(FConsoleVariableDelegate::CreateStatic(&OnChangeShadingQuality));
	CVarLandscapeQuality.AsVariable()->SetOnChangedCallback(FConsoleVariableDelegate::CreateStatic(&OnChangeLandscapeQuality));

#if WITH_EDITOR
	ScalabilityShaderPlatform = GMaxRHIShaderPlatform;
	bScalabilityShaderPlatformHasBeenChanged = false;
#endif
	// Set defaults
	SetQualityLevels(FQualityLevels());
	GScalabilityBackupQualityLevels = FQualityLevels();
	GScalabilityQualityLevelsOverride = FQualityLevels();
	GScalabilityUsingTemporaryQualityLevels = false;
	bIsInitialized = true;
}

/** Get the percentage scale for a given quality level */
static float GetRenderScaleLevelFromQualityLevel(int32 InQualityLevel, EQualityLevelBehavior Behavior = EQualityLevelBehavior::EAbsolute)
{
	TArray<FString> ResolutionValueStrings;
	GConfig->GetSingleLineArray(TEXT("ScalabilitySettings"), TEXT("PerfIndexValues_ResolutionQuality"), ResolutionValueStrings, GScalabilityIni);

	if (ResolutionValueStrings.Num() == 0)
	{
		UE_LOGF(LogConsoleResponse, Display, "Failed to find resolution value strings in scalability ini. Falling back to default.");
		return 100.0f;
	}

	//no negative levels.
	InQualityLevel = FMath::Max(0, InQualityLevel);
	if (Behavior == EQualityLevelBehavior::ERelativeToMax)
	{
		InQualityLevel = FMath::Max(ResolutionValueStrings.Num() - 1 - InQualityLevel, 0);
	}
	else
	{
		InQualityLevel = FMath::Clamp(InQualityLevel, 0, ResolutionValueStrings.Num() - 1);
	}

	return FCString::Atof(*ResolutionValueStrings[InQualityLevel]);
}

TArray<FResolutionPreset> GetResolutionPresets()
{
	TArray<FResolutionPreset> Presets;
	TArray<FString> AllPresetStrings;
	GConfig->GetArray(TEXT("ResolutionQuality"), TEXT("ResolutionPresets"), /* out */ AllPresetStrings, GScalabilityIni);
	for (FString PresetString : AllPresetStrings)
	{
		// Remove parentheses
		PresetString.TrimStartAndEndInline();
		PresetString.ReplaceInline(TEXT("("), TEXT(""));
		PresetString.ReplaceInline(TEXT(")"), TEXT(""));

		// Parse all properties
		FResolutionPreset Preset;
		bool bSuccess = true;

		bSuccess &= FParse::Value(*PresetString, TEXT("Name="), /* out */ Preset.Name);
		ensure(bSuccess);
		bSuccess &= FParse::Value(*PresetString, TEXT("ResolutionQuality="), /* out */ Preset.ResolutionQuality);
		ensure(bSuccess);

		if (bSuccess)
		{
			Preset.Id = Presets.Num();
			Presets.Add(Preset);
		}
		else
		{
			ensure(false);
		}
	}

	return Presets;
}

ENGINE_API float GetResolutionQualityFromGPUPerfIndex(float GPUPerfIndex)
{
	 return GetRenderScaleLevelFromQualityLevel(ComputeOptionFromPerfIndex(TEXT("ResolutionQuality"), 0, GPUPerfIndex));
}

ENGINE_API FQualityLevels ComputeQualityLevelsFromPerfIndex(float CPUPerfIndex, float GPUPerfIndex)
{
	FQualityLevels Results;
	Results.ResolutionQuality = GetResolutionQualityFromGPUPerfIndex(GPUPerfIndex);
	Results.ViewDistanceQuality = ComputeOptionFromPerfIndex(TEXT("ViewDistanceQuality"), CPUPerfIndex, GPUPerfIndex);
	Results.AntiAliasingQuality = ComputeOptionFromPerfIndex(TEXT("AntiAliasingQuality"), CPUPerfIndex, GPUPerfIndex);
	Results.ShadowQuality = ComputeOptionFromPerfIndex(TEXT("ShadowQuality"), CPUPerfIndex, GPUPerfIndex);
	Results.GlobalIlluminationQuality = ComputeOptionFromPerfIndex(TEXT("GlobalIlluminationQuality"), CPUPerfIndex, GPUPerfIndex);
	Results.ReflectionQuality = ComputeOptionFromPerfIndex(TEXT("ReflectionQuality"), CPUPerfIndex, GPUPerfIndex);
	Results.PostProcessQuality = ComputeOptionFromPerfIndex(TEXT("PostProcessQuality"), CPUPerfIndex, GPUPerfIndex);
	Results.TextureQuality = ComputeOptionFromPerfIndex(TEXT("TextureQuality"), CPUPerfIndex, GPUPerfIndex);
	Results.EffectsQuality = ComputeOptionFromPerfIndex(TEXT("EffectsQuality"), CPUPerfIndex, GPUPerfIndex);
	Results.FoliageQuality = ComputeOptionFromPerfIndex(TEXT("FoliageQuality"), CPUPerfIndex, GPUPerfIndex);
	Results.ShadingQuality = ComputeOptionFromPerfIndex(TEXT("ShadingQuality"), CPUPerfIndex, GPUPerfIndex);
	Results.LandscapeQuality = ComputeOptionFromPerfIndex( TEXT("LandscapeQuality"), CPUPerfIndex, GPUPerfIndex);
	Results.CPUBenchmarkResults = CPUPerfIndex;
	Results.GPUBenchmarkResults = GPUPerfIndex;
	return Results;
}

FQualityLevels BenchmarkQualityLevels(uint32 WorkScale, float CPUMultiplier, float GPUMultiplier)
{
	ensure((CPUMultiplier > 0.0f) && (GPUMultiplier > 0.0f));

	// benchmark the system

	TArray<float> CPUBenchmarkSteps;
	TArray<float> GPUBenchmarkSteps;

	FSynthBenchmarkResults SynthBenchmark;
	ISynthBenchmark::Get().Run(SynthBenchmark, true, WorkScale);

	float CPUPerfIndex = SynthBenchmark.ComputeCPUPerfIndex(/*out*/ &CPUBenchmarkSteps) * CPUMultiplier;
	float GPUPerfIndex = SynthBenchmark.ComputeGPUPerfIndex(/*out*/ &GPUBenchmarkSteps) * GPUMultiplier;

#if !UE_BUILD_SHIPPING
	if (CVarTestCPUPerfIndexOverride.GetValueOnAnyThread() > 0.0f)
	{
		CPUPerfIndex = CVarTestCPUPerfIndexOverride.GetValueOnAnyThread();
	}
	if (CVarTestGPUPerfIndexOverride.GetValueOnAnyThread() > 0.0f)
	{
		GPUPerfIndex = CVarTestGPUPerfIndexOverride.GetValueOnAnyThread();
	}
#endif

	// decide on the actual quality needed
	FQualityLevels Results = ComputeQualityLevelsFromPerfIndex(CPUPerfIndex, GPUPerfIndex);
	Results.CPUBenchmarkSteps = MoveTemp(CPUBenchmarkSteps);
	Results.GPUBenchmarkSteps = MoveTemp(GPUBenchmarkSteps);

	return Results;
}

static void PrintGroupInfo(const TCHAR* InGroupName, bool bInInfoMode)
{
	int32 QualityLevel = -1;
	TArray<FString> CVars;
	InferCurrentQualityLevel(InGroupName, QualityLevel, &CVars);

	FString GroupQualityLevelDisplayName = QualityLevel == -1 ? TEXT("(custom)") : FString::FromInt(QualityLevel);

	UE_LOGF(LogConsoleResponse, Display, "  %ls (0..3): %ls", InGroupName, *GroupQualityLevelDisplayName);

	if (bInInfoMode)
	{
		if (QualityLevel != -1)
		{
			for (int32 CVarDataIndex = 0; CVarDataIndex < CVars.Num(); ++CVarDataIndex)
			{
				UE_LOGF(LogConsoleResponse, Display, "    %ls", *CVars[CVarDataIndex]);
			}
		}
	}
}

void ProcessCommand(const TCHAR* Cmd, FOutputDevice& Ar)
{
	bool bPrintUsage = true;
	bool bPrintCurrentSettings = true;
	bool bInfoMode = false;
	FString Token;	

	float CPUBenchmarkValue = -1.0f;
	float GPUBenchmarkValue = -1.0f;

	// Parse command line
	if (FParse::Token(Cmd, Token, true))
	{
		if (Token == TEXT("auto"))
		{
			FQualityLevels State = Scalability::BenchmarkQualityLevels();
			Scalability::SetQualityLevels(State);
			Scalability::SaveState(GIsEditor ? GEditorSettingsIni : GGameUserSettingsIni);
			bPrintUsage = false;
			bPrintCurrentSettings = true;
			CPUBenchmarkValue = State.CPUBenchmarkResults;
			GPUBenchmarkValue = State.GPUBenchmarkResults;
		}
		else if (Token == TEXT("reapply"))
		{
			FQualityLevels State = Scalability::GetQualityLevels();
			Scalability::SetQualityLevels(State);
			bPrintUsage = false;
		}
		else if (Token == TEXT("cine"))
		{
			FQualityLevels QualityLevels;			
			QualityLevels.SetFromSingleQualityLevel(MAX_int32);
			SetQualityLevels(QualityLevels);
			Scalability::SaveState(GIsEditor ? GEditorSettingsIni : GGameUserSettingsIni);

			bPrintUsage = false;
		}
		else if (Token.IsNumeric())
		{
			FQualityLevels QualityLevels;

			int32 RequestedQualityLevel = FCString::Atoi(*Token);
			QualityLevels.SetFromSingleQualityLevel(RequestedQualityLevel);
			SetQualityLevels(QualityLevels);
			Scalability::SaveState(GIsEditor ? GEditorSettingsIni : GGameUserSettingsIni);

			bPrintUsage = false;
		}
		else
		{
			UE_LOGF(LogConsoleResponse, Error, "Scalability unknown parameter");
			bPrintCurrentSettings = false;
		}
	}

	if (bPrintUsage)
	{
		UE_LOGF(LogConsoleResponse, Display, "Scalability Usage:");
		UE_LOGF(LogConsoleResponse, Display, "  \"Scalability\" (Print scalability usage and information)");
		UE_LOGF(LogConsoleResponse, Display, "  \"Scalability [0..3]\" (Set all scalability groups to the specified quality level and save state)");
		UE_LOGF(LogConsoleResponse, Display, "  \"Scalability reapply\" (apply the state of the scalability group (starting with 'sg.') console variables)");
		UE_LOGF(LogConsoleResponse, Display, "  \"Scalability auto\" (Run synth benchmark and adjust the scalability levels for your system and save state)");
	}

	if (bPrintCurrentSettings)
	{
		UE_LOGF(LogConsoleResponse, Display, "Current Scalability Settings:");

		PrintGroupInfo(TEXT("ResolutionQuality"), bInfoMode);
		PrintGroupInfo(TEXT("ViewDistanceQuality"), bInfoMode);
		PrintGroupInfo(TEXT("AntiAliasingQuality"), bInfoMode);
		PrintGroupInfo(TEXT("ShadowQuality"), bInfoMode);
		PrintGroupInfo(TEXT("GlobalIlluminationQuality"), bInfoMode);
		PrintGroupInfo(TEXT("ReflectionQuality"), bInfoMode);
		PrintGroupInfo(TEXT("PostProcessQuality"), bInfoMode);
		PrintGroupInfo(TEXT("TextureQuality"), bInfoMode);
		PrintGroupInfo(TEXT("EffectsQuality"), bInfoMode);
		PrintGroupInfo(TEXT("FoliageQuality"), bInfoMode);
		PrintGroupInfo(TEXT("ShadingQuality"), bInfoMode);

		if (CPUBenchmarkValue >= 0.0f)
		{
			UE_LOGF(LogConsoleResponse, Display, "CPU benchmark value: %f", CPUBenchmarkValue);
		}
		if (GPUBenchmarkValue >= 0.0f)
		{
			UE_LOGF(LogConsoleResponse, Display, "GPU benchmark value: %f", GPUBenchmarkValue);
		}
	}
}

template<typename T> void SetQualityLevelCVar(TAutoConsoleVariable<T>& CVarToEdit, const T DesiredValue, const T OverrideValue, bool bForce)
{
	if (!bForce && GScalabilityUsingTemporaryQualityLevels && (OverrideValue >= T(0)))
	{
		// If we are alredy overriding the value with a valid temporary one, don't set it.
		return;
	}

	if (bForce)
	{
		CVarToEdit.AsVariable()->SetWithCurrentPriority(DesiredValue);
	}
#if WITH_EDITOR
	else if (GEditor && GEditor->IsPreviewPlatformActive())
	{
		EConsoleVariableFlags MaxPriority = ECVF_SetByPreview;
		EConsoleVariableFlags MinPriority = ECVF_SetByScalability;
		CVarToEdit.AsVariable()->ReplaceCurrentPriorityAndTag(DesiredValue, MaxPriority, MinPriority);
	}
#endif
	else
	{
		T PreviousValue;
		CVarToEdit.AsVariable()->GetValue(PreviousValue);
		if (DesiredValue != PreviousValue || !bIsInitialized)
		{
			CVarToEdit.AsVariable()->Set(DesiredValue, ECVF_SetByScalability);
		}
	}
}

void SetQualityLevels(const FQualityLevels& QualityLevels, bool bForce/* = false*/)
{
	// The setter functions clamp these values internally
	FQualityLevels ClampedLevels;
	ClampedLevels.ResolutionQuality = QualityLevels.ResolutionQuality;
	ClampedLevels.SetViewDistanceQuality(QualityLevels.ViewDistanceQuality);
	ClampedLevels.SetAntiAliasingQuality(QualityLevels.AntiAliasingQuality);
	ClampedLevels.SetShadowQuality(QualityLevels.ShadowQuality);
	ClampedLevels.SetGlobalIlluminationQuality(QualityLevels.GlobalIlluminationQuality);
	ClampedLevels.SetReflectionQuality(QualityLevels.ReflectionQuality);
	ClampedLevels.SetPostProcessQuality(QualityLevels.PostProcessQuality);
	ClampedLevels.SetTextureQuality(QualityLevels.TextureQuality);
	ClampedLevels.SetEffectsQuality(QualityLevels.EffectsQuality);
	ClampedLevels.SetFoliageQuality(QualityLevels.FoliageQuality);
	ClampedLevels.SetShadingQuality(QualityLevels.ShadingQuality);
	ClampedLevels.SetLandscapeQuality(QualityLevels.LandscapeQuality);

	if (GScalabilityUsingTemporaryQualityLevels && !bForce)
	{
		// When temporary scalability is active, non-temporary sets are
		// applied to the backup levels so we can restore them later
		GScalabilityBackupQualityLevels = ClampedLevels;
	}

	//Skip the broadcast and assignations if the levels were the same.
	if (GetQualityLevels() != ClampedLevels || bForce || !bIsInitialized)
	{
		SetQualityLevelCVar(CVarResolutionQuality, ClampedLevels.ResolutionQuality, GScalabilityQualityLevelsOverride.ResolutionQuality, bForce);
		SetQualityLevelCVar(CVarViewDistanceQuality, ClampedLevels.ViewDistanceQuality, GScalabilityQualityLevelsOverride.ViewDistanceQuality, bForce);
		SetQualityLevelCVar(CVarAntiAliasingQuality, ClampedLevels.AntiAliasingQuality, GScalabilityQualityLevelsOverride.AntiAliasingQuality, bForce);
		SetQualityLevelCVar(CVarShadowQuality, ClampedLevels.ShadowQuality, GScalabilityQualityLevelsOverride.ShadowQuality, bForce);
		SetQualityLevelCVar(CVarGlobalIlluminationQuality, ClampedLevels.GlobalIlluminationQuality, GScalabilityQualityLevelsOverride.GlobalIlluminationQuality, bForce);
		SetQualityLevelCVar(CVarReflectionQuality, ClampedLevels.ReflectionQuality, GScalabilityQualityLevelsOverride.ReflectionQuality, bForce);
		SetQualityLevelCVar(CVarPostProcessQuality, ClampedLevels.PostProcessQuality, GScalabilityQualityLevelsOverride.PostProcessQuality, bForce);
		SetQualityLevelCVar(CVarTextureQuality, ClampedLevels.TextureQuality, GScalabilityQualityLevelsOverride.TextureQuality, bForce);
		SetQualityLevelCVar(CVarEffectsQuality, ClampedLevels.EffectsQuality, GScalabilityQualityLevelsOverride.EffectsQuality, bForce);
		SetQualityLevelCVar(CVarFoliageQuality, ClampedLevels.FoliageQuality, GScalabilityQualityLevelsOverride.FoliageQuality, bForce);
		SetQualityLevelCVar(CVarShadingQuality, ClampedLevels.ShadingQuality, GScalabilityQualityLevelsOverride.ShadingQuality, bForce);
		SetQualityLevelCVar(CVarLandscapeQuality, ClampedLevels.LandscapeQuality, GScalabilityQualityLevelsOverride.LandscapeQuality, bForce);

		OnScalabilitySettingsChanged.Broadcast(ClampedLevels);
	}

#if WITH_EDITOR
	// We directly use UDeviceProfileManager::DeviceProfileManagerSingleton here so we don't unexpectedly initialize the device profile system during cooking.
	if (ScalabilityShaderPlatform != EShaderPlatform::SP_NumPlatforms && UDeviceProfileManager::DeviceProfileManagerSingleton)
	{
		const UDeviceProfile* PreviewDeviceProfile = UDeviceProfileManager::DeviceProfileManagerSingleton->GetPreviewDeviceProfile();
		if (PreviewDeviceProfile)
		{
			PreviewShaderPlatformToCachedQualityLevels.FindOrAdd(MakeTuple(ScalabilityShaderPlatform, PreviewDeviceProfile->GetFName())) = ClampedLevels;
		}
	}
	bScalabilityShaderPlatformHasBeenChanged = false;
#endif
}

FQualityLevels GetQualityLevels()
{
	FQualityLevels Ret;

	// Only suggested way to get the current state - don't get CVars directly
	if (!GScalabilityUsingTemporaryQualityLevels)
	{
		Ret.ResolutionQuality = CVarResolutionQuality.GetValueOnGameThread();
		Ret.ViewDistanceQuality = CVarViewDistanceQuality.GetValueOnGameThread();
		Ret.AntiAliasingQuality = CVarAntiAliasingQuality.GetValueOnGameThread();
		Ret.ShadowQuality = CVarShadowQuality.GetValueOnGameThread();
		Ret.GlobalIlluminationQuality = CVarGlobalIlluminationQuality.GetValueOnGameThread();
		Ret.ReflectionQuality = CVarReflectionQuality.GetValueOnGameThread();
		Ret.PostProcessQuality = CVarPostProcessQuality.GetValueOnGameThread();
		Ret.TextureQuality = CVarTextureQuality.GetValueOnGameThread();
		Ret.EffectsQuality = CVarEffectsQuality.GetValueOnGameThread();
		Ret.FoliageQuality = CVarFoliageQuality.GetValueOnGameThread();
		Ret.ShadingQuality = CVarShadingQuality.GetValueOnGameThread();
		Ret.LandscapeQuality = CVarLandscapeQuality.GetValueOnGameThread();
	}
	else
	{
		Ret = GScalabilityBackupQualityLevels;
	}

	return Ret;
}

void ToggleTemporaryQualityLevels(bool bEnable, const FQualityLevels& QualityLevelsOverride)
{
	if (bEnable != GScalabilityUsingTemporaryQualityLevels)
	{
		if (!GScalabilityUsingTemporaryQualityLevels)
		{
			GScalabilityBackupQualityLevels = GetQualityLevels();
			GScalabilityQualityLevelsOverride = QualityLevelsOverride;
			GScalabilityUsingTemporaryQualityLevels = true;
		}
		else
		{
			GScalabilityUsingTemporaryQualityLevels = false;
			GScalabilityQualityLevelsOverride = FQualityLevels();
			SetQualityLevels(GScalabilityBackupQualityLevels, true);
		}
	}
}

bool IsTemporaryQualityLevelActive()
{
	return GScalabilityUsingTemporaryQualityLevels;
}

int32 GetEffectsQualityDirect(bool bGameThread)
{
	if (bGameThread)
	{
		return CVarEffectsQuality.GetValueOnAnyThread(true);
	}
	else
	{
		return CVarEffectsQuality.GetValueOnRenderThread();
	}
}

void FQualityLevels::SetBenchmarkFallback()
{
	ResolutionQuality = 100.0f;
}

void FQualityLevels::SetDefaults()
{
	// Clamp to Epic (Max-1) settings, we don't allow Cinematic (Max) quality by default
	SetFromSingleQualityLevelRelativeToMax(1);
	ResolutionQuality = 0.0f; // fall back to project's default screen percentage
}

void FQualityLevels::SetFromSingleQualityLevel(int32 Value)
{
	ResolutionQuality = GetRenderScaleLevelFromQualityLevel(Value, EQualityLevelBehavior::EAbsolute);
	ViewDistanceQuality = FMath::Clamp(Value, 0, CVarViewDistanceQuality_NumLevels->GetInt() - 1);
	AntiAliasingQuality = FMath::Clamp(Value, 0, CVarAntiAliasingQuality_NumLevels->GetInt() - 1);
	ShadowQuality = FMath::Clamp(Value, 0, CVarShadowQuality_NumLevels->GetInt() - 1);
	GlobalIlluminationQuality = FMath::Clamp(Value, 0, CVarGlobalIlluminationQuality_NumLevels->GetInt() - 1);
	ReflectionQuality = FMath::Clamp(Value, 0, CVarReflectionQuality_NumLevels->GetInt() - 1);
	PostProcessQuality = FMath::Clamp(Value, 0, CVarPostProcessQuality_NumLevels->GetInt() - 1);
	TextureQuality = FMath::Clamp(Value, 0, CVarTextureQuality_NumLevels->GetInt() - 1);
	EffectsQuality = FMath::Clamp(Value, 0, CVarEffectsQuality_NumLevels->GetInt() - 1);
	FoliageQuality = FMath::Clamp(Value, 0, CVarFoliageQuality_NumLevels->GetInt() - 1);
	ShadingQuality = FMath::Clamp(Value, 0, CVarShadingQuality_NumLevels->GetInt() - 1);
	LandscapeQuality = FMath::Clamp(Value, 0, CVarLandscapeQuality_NumLevels->GetInt() - 1);
}

void FQualityLevels::SetFromSingleQualityLevelRelativeToMax(int32 Value)
{
	ResolutionQuality = GetRenderScaleLevelFromQualityLevel(Value, EQualityLevelBehavior::ERelativeToMax);

	//account for 0 indexing.
	Value += 1;

	ViewDistanceQuality = FMath::Max(CVarViewDistanceQuality_NumLevels->GetInt() - Value, 0);
	AntiAliasingQuality = FMath::Max(CVarAntiAliasingQuality_NumLevels->GetInt() - Value, 0);
	ShadowQuality = FMath::Max(CVarShadowQuality_NumLevels->GetInt() - Value, 0);
	GlobalIlluminationQuality = FMath::Max(CVarGlobalIlluminationQuality_NumLevels->GetInt() - Value, 0);
	ReflectionQuality = FMath::Max(CVarReflectionQuality_NumLevels->GetInt() - Value, 0);
	PostProcessQuality = FMath::Max(CVarPostProcessQuality_NumLevels->GetInt() - Value, 0);
	TextureQuality = FMath::Max(CVarTextureQuality_NumLevels->GetInt() - Value, 0);
	EffectsQuality = FMath::Max(CVarEffectsQuality_NumLevels->GetInt() - Value, 0);
	FoliageQuality = FMath::Max(CVarFoliageQuality_NumLevels->GetInt() - Value, 0);
	ShadingQuality = FMath::Max(CVarShadingQuality_NumLevels->GetInt() - Value, 0);
	LandscapeQuality =  FMath::Max(CVarLandscapeQuality_NumLevels->GetInt() - Value, 0);
}

// Returns the overall value if all settings are set to the same thing
// @param Value -1:custom 0:low, 1:medium, 2:high, 3:epic
int32 FQualityLevels::GetSingleQualityLevel() const
{
	int32 Result = ViewDistanceQuality;

	const int32 Target = ViewDistanceQuality;
	if ((Target == AntiAliasingQuality) && (Target == ShadowQuality) && (Target == GlobalIlluminationQuality) && (Target == ReflectionQuality) && (Target == PostProcessQuality) && (Target == TextureQuality) && (Target == EffectsQuality) && (Target == FoliageQuality) && (Target == ShadingQuality) && (Target == LandscapeQuality))
	{
		if (GetRenderScaleLevelFromQualityLevel(Target) == ResolutionQuality)
		{
			return Target;
		}
	}

	return -1;
}

int32 FQualityLevels::GetMinQualityLevel() const
{
	int32 Level = ViewDistanceQuality;

	Level = FMath::Min(Level, AntiAliasingQuality);
	Level = FMath::Min(Level, ShadowQuality);
	Level = FMath::Min(Level, GlobalIlluminationQuality);
	Level = FMath::Min(Level, ReflectionQuality);
	Level = FMath::Min(Level, PostProcessQuality);
	Level = FMath::Min(Level, TextureQuality);
	Level = FMath::Min(Level, EffectsQuality);
	Level = FMath::Min(Level, FoliageQuality);
	Level = FMath::Min(Level, ShadingQuality);
	Level = FMath::Min(Level, LandscapeQuality);

	return Level;
}

void FQualityLevels::SetViewDistanceQuality(int32 Value)
{
	ViewDistanceQuality = FMath::Clamp(Value, 0, CVarViewDistanceQuality_NumLevels->GetInt() - 1);
}

void FQualityLevels::SetAntiAliasingQuality(int32 Value)
{
	AntiAliasingQuality = FMath::Clamp(Value, 0, CVarAntiAliasingQuality_NumLevels->GetInt() - 1);
}

void FQualityLevels::SetShadowQuality(int32 Value)
{
	ShadowQuality = FMath::Clamp(Value, 0, CVarShadowQuality_NumLevels->GetInt() - 1);
}

void FQualityLevels::SetGlobalIlluminationQuality(int32 Value)
{
	GlobalIlluminationQuality = FMath::Clamp(Value, 0, CVarGlobalIlluminationQuality_NumLevels->GetInt() - 1);
}

void FQualityLevels::SetReflectionQuality(int32 Value)
{
	ReflectionQuality = FMath::Clamp(Value, 0, CVarReflectionQuality_NumLevels->GetInt() - 1);
}

void FQualityLevels::SetPostProcessQuality(int32 Value)
{
	PostProcessQuality = FMath::Clamp(Value, 0, CVarPostProcessQuality_NumLevels->GetInt() - 1);
}

void FQualityLevels::SetTextureQuality(int32 Value)
{
	TextureQuality = FMath::Clamp(Value, 0, CVarTextureQuality_NumLevels->GetInt() - 1);
}

void FQualityLevels::SetEffectsQuality(int32 Value)
{
	EffectsQuality = FMath::Clamp(Value, 0, CVarEffectsQuality_NumLevels->GetInt() - 1);
}

void FQualityLevels::SetFoliageQuality(int32 Value)
{
	FoliageQuality = FMath::Clamp(Value, 0, CVarFoliageQuality_NumLevels->GetInt() - 1);
}

void FQualityLevels::SetShadingQuality(int32 Value)
{
	ShadingQuality = FMath::Clamp(Value, 0, CVarShadingQuality_NumLevels->GetInt() - 1);
}

void FQualityLevels::SetLandscapeQuality(int32 Value)
{
	LandscapeQuality = FMath::Clamp(Value, 0, CVarLandscapeQuality_NumLevels->GetInt() - 1);
}

void LoadState(const FString& IniName)
{
	LoadState(IniName, TEXT("ScalabilityGroups"));
}
	
void LoadState(const FString& IniName, const TCHAR* Section)
{
	check(!IniName.IsEmpty());

	// todo: could be done earlier
	InitScalabilitySystem();

	// Use existing quality levels - Defaults with device profile customization
	FQualityLevels State = GetQualityLevels();

	// looks like cvars but here we just use the name for the ini
	GConfig->GetFloat(Section, TEXT("sg.ResolutionQuality"), State.ResolutionQuality, IniName);
	GConfig->GetInt(Section, TEXT("sg.ViewDistanceQuality"), State.ViewDistanceQuality, IniName);
	GConfig->GetInt(Section, TEXT("sg.AntiAliasingQuality"), State.AntiAliasingQuality, IniName);
	GConfig->GetInt(Section, TEXT("sg.ShadowQuality"), State.ShadowQuality, IniName);
	GConfig->GetInt(Section, TEXT("sg.GlobalIlluminationQuality"), State.GlobalIlluminationQuality, IniName);
	GConfig->GetInt(Section, TEXT("sg.ReflectionQuality"), State.ReflectionQuality, IniName);
	GConfig->GetInt(Section, TEXT("sg.PostProcessQuality"), State.PostProcessQuality, IniName);
	GConfig->GetInt(Section, TEXT("sg.TextureQuality"), State.TextureQuality, IniName);
	GConfig->GetInt(Section, TEXT("sg.EffectsQuality"), State.EffectsQuality, IniName);
	GConfig->GetInt(Section, TEXT("sg.FoliageQuality"), State.FoliageQuality, IniName);
	GConfig->GetInt(Section, TEXT("sg.ShadingQuality"), State.ShadingQuality, IniName);
	GConfig->GetInt(Section, TEXT("sg.LandscapeQuality"), State.LandscapeQuality, IniName);

	// If possible apply immediately, else store in backup so we can re-apply later
	if (!GScalabilityUsingTemporaryQualityLevels)
	{
		SetQualityLevels(State);
	}
	else
	{
		GScalabilityBackupQualityLevels = State;
	}
}

void SaveState(const FString& IniName)
{
	// Saves to the previous default
	SaveState(IniName, TEXT("ScalabilityGroups"));
}

void SaveState(const FString& IniName, const TCHAR* Section)
{
	check(!IniName.IsEmpty());

	// Save the "real" settings if in a temporary state
	FQualityLevels State = GScalabilityUsingTemporaryQualityLevels ? GScalabilityBackupQualityLevels : GetQualityLevels();

	// looks like cvars but here we just use the name for the ini
	GConfig->SetFloat(Section, TEXT("sg.ResolutionQuality"), State.ResolutionQuality, IniName);
	GConfig->SetInt(Section, TEXT("sg.ViewDistanceQuality"), State.ViewDistanceQuality, IniName);
	GConfig->SetInt(Section, TEXT("sg.AntiAliasingQuality"), State.AntiAliasingQuality, IniName);
	GConfig->SetInt(Section, TEXT("sg.ShadowQuality"), State.ShadowQuality, IniName);
	GConfig->SetInt(Section, TEXT("sg.GlobalIlluminationQuality"), State.GlobalIlluminationQuality, IniName);
	GConfig->SetInt(Section, TEXT("sg.ReflectionQuality"), State.ReflectionQuality, IniName);
	GConfig->SetInt(Section, TEXT("sg.PostProcessQuality"), State.PostProcessQuality, IniName);
	GConfig->SetInt(Section, TEXT("sg.TextureQuality"), State.TextureQuality, IniName);
	GConfig->SetInt(Section, TEXT("sg.EffectsQuality"), State.EffectsQuality, IniName);
	GConfig->SetInt(Section, TEXT("sg.FoliageQuality"), State.FoliageQuality, IniName);
	GConfig->SetInt(Section, TEXT("sg.ShadingQuality"), State.ShadingQuality, IniName);
	GConfig->SetInt(Section, TEXT("sg.LandscapeQuality"), State.LandscapeQuality, IniName);
}

void RecordQualityLevelsAnalytics(bool bAutoApplied)
{
	if( FEngineAnalytics::IsAvailable() )
	{
		FQualityLevels State = GetQualityLevels();

		TArray<FAnalyticsEventAttribute> Attributes;

		Attributes.Add(FAnalyticsEventAttribute(TEXT("ResolutionQuality"), State.ResolutionQuality));
		Attributes.Add(FAnalyticsEventAttribute(TEXT("ViewDistanceQuality"), State.ViewDistanceQuality));
		Attributes.Add(FAnalyticsEventAttribute(TEXT("AntiAliasingQuality"), State.AntiAliasingQuality));
		Attributes.Add(FAnalyticsEventAttribute(TEXT("ShadowQuality"), State.ShadowQuality));
		Attributes.Add(FAnalyticsEventAttribute(TEXT("GlobalIlluminationQuality"), State.GlobalIlluminationQuality));
		Attributes.Add(FAnalyticsEventAttribute(TEXT("ReflectionQuality"), State.ReflectionQuality));
		Attributes.Add(FAnalyticsEventAttribute(TEXT("PostProcessQuality"), State.PostProcessQuality));
		Attributes.Add(FAnalyticsEventAttribute(TEXT("TextureQuality"), State.TextureQuality));
		Attributes.Add(FAnalyticsEventAttribute(TEXT("EffectsQuality"), State.EffectsQuality));
		Attributes.Add(FAnalyticsEventAttribute(TEXT("FoliageQuality"), State.FoliageQuality));
		Attributes.Add(FAnalyticsEventAttribute(TEXT("ShadingQuality"), State.ShadingQuality));
		Attributes.Add( FAnalyticsEventAttribute(TEXT("LandscapeQuality"), State.LandscapeQuality));
		Attributes.Add(FAnalyticsEventAttribute(TEXT("AutoAppliedSettings"), bAutoApplied));
		Attributes.Add(FAnalyticsEventAttribute(TEXT("Enterprise"), IProjectManager::Get().IsEnterpriseProject()));

		FEngineAnalytics::GetProvider().RecordEvent(TEXT("Editor.Performance.ScalabiltySettings"), Attributes);
	}
}

FQualityLevels GetQualityLevelCounts()
{
	FQualityLevels Result;
	Result.ResolutionQuality = 100.0f;
	Result.ViewDistanceQuality = CVarViewDistanceQuality_NumLevels->GetInt();
	Result.AntiAliasingQuality = CVarAntiAliasingQuality_NumLevels->GetInt();
	Result.ShadowQuality = CVarShadowQuality_NumLevels->GetInt();
	Result.GlobalIlluminationQuality = CVarGlobalIlluminationQuality_NumLevels->GetInt();
	Result.ReflectionQuality = CVarReflectionQuality_NumLevels->GetInt();
	Result.PostProcessQuality = CVarPostProcessQuality_NumLevels->GetInt();
	Result.TextureQuality = CVarTextureQuality_NumLevels->GetInt();
	Result.EffectsQuality = CVarEffectsQuality_NumLevels->GetInt();
	Result.FoliageQuality = CVarFoliageQuality_NumLevels->GetInt();
	Result.ShadingQuality = CVarShadingQuality_NumLevels->GetInt();
	Result.LandscapeQuality = CVarLandscapeQuality_NumLevels->GetInt();
	return Result;
}

void LoadPlatformScalability(FString PlatformName)
{
}

#define LOCTEXT_NAMESPACE "Scalability"

FText GetQualityLevelText(int32 QualityLevel, int32 NumLevels)
{
	//This matches logic in editor scalability settings UI. TODO: Unify.
	const FText Names[5] = { LOCTEXT("QualityLowLabel", "Low"), LOCTEXT("QualityMediumLabel", "Medium"), LOCTEXT("QualityHighLabel", "High"), LOCTEXT("QualityEpicLabel", "Epic"), LOCTEXT("QualityCineLabel", "Cinematic") };

	QualityLevel = FMath::Clamp(QualityLevel, 0, NumLevels - 1);
	if (NumLevels == 5)
	{
		return Names[QualityLevel];
	}
	else
	{
		if (QualityLevel == NumLevels - 1)
		{
			return Names[4];
		}
		else
		{
			return FText::AsNumber(QualityLevel);
		}
	}
}

#undef LOCTEXT_NAMESPACE

}

