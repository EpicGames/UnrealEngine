// Copyright Epic Games, Inc. All Rights Reserved.

#include "Engine/Scene.h"
#include "Engine/EngineTypes.h"
#include "HAL/IConsoleManager.h"
#include "RenderUtils.h"
#include "UObject/RenderingObjectVersion.h"
#include "UObject/ReleaseObjectVersion.h"
#include "UObject/UE5ReleaseStreamObjectVersion.h"
#include "UObject/FortniteMainBranchObjectVersion.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "UObject/UnrealType.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(Scene)

void FColorGradingSettings::ExportToPostProcessSettings(FPostProcessSettings* OutPostProcessSettings) const
{
	OutPostProcessSettings->bOverride_ColorSaturation = true;
	OutPostProcessSettings->bOverride_ColorContrast = true;
	OutPostProcessSettings->bOverride_ColorGamma = true;
	OutPostProcessSettings->bOverride_ColorGain = true;
	OutPostProcessSettings->bOverride_ColorOffset = true;

	OutPostProcessSettings->bOverride_ColorSaturationShadows = true;
	OutPostProcessSettings->bOverride_ColorContrastShadows = true;
	OutPostProcessSettings->bOverride_ColorGammaShadows = true;
	OutPostProcessSettings->bOverride_ColorGainShadows = true;
	OutPostProcessSettings->bOverride_ColorOffsetShadows = true;

	OutPostProcessSettings->bOverride_ColorSaturationMidtones = true;
	OutPostProcessSettings->bOverride_ColorContrastMidtones = true;
	OutPostProcessSettings->bOverride_ColorGammaMidtones = true;
	OutPostProcessSettings->bOverride_ColorGainMidtones = true;
	OutPostProcessSettings->bOverride_ColorOffsetMidtones = true;

	OutPostProcessSettings->bOverride_ColorSaturationHighlights = true;
	OutPostProcessSettings->bOverride_ColorContrastHighlights = true;
	OutPostProcessSettings->bOverride_ColorGammaHighlights = true;
	OutPostProcessSettings->bOverride_ColorGainHighlights = true;
	OutPostProcessSettings->bOverride_ColorOffsetHighlights = true;

	OutPostProcessSettings->bOverride_ColorCorrectionShadowsMax = true;
	OutPostProcessSettings->bOverride_ColorCorrectionHighlightsMin = true;
	OutPostProcessSettings->bOverride_ColorCorrectionHighlightsMax = true;

	OutPostProcessSettings->ColorSaturation = Global.Saturation;
	OutPostProcessSettings->ColorContrast = Global.Contrast;
	OutPostProcessSettings->ColorGamma = Global.Gamma;
	OutPostProcessSettings->ColorGain = Global.Gain;
	OutPostProcessSettings->ColorOffset = Global.Offset;

	OutPostProcessSettings->ColorSaturationShadows = Shadows.Saturation;
	OutPostProcessSettings->ColorContrastShadows = Shadows.Contrast;
	OutPostProcessSettings->ColorGammaShadows = Shadows.Gamma;
	OutPostProcessSettings->ColorGainShadows = Shadows.Gain;
	OutPostProcessSettings->ColorOffsetShadows = Shadows.Offset;

	OutPostProcessSettings->ColorSaturationMidtones = Midtones.Saturation;
	OutPostProcessSettings->ColorContrastMidtones = Midtones.Contrast;
	OutPostProcessSettings->ColorGammaMidtones = Midtones.Gamma;
	OutPostProcessSettings->ColorGainMidtones = Midtones.Gain;
	OutPostProcessSettings->ColorOffsetMidtones = Midtones.Offset;

	OutPostProcessSettings->ColorSaturationHighlights = Highlights.Saturation;
	OutPostProcessSettings->ColorContrastHighlights = Highlights.Contrast;
	OutPostProcessSettings->ColorGammaHighlights = Highlights.Gamma;
	OutPostProcessSettings->ColorGainHighlights = Highlights.Gain;
	OutPostProcessSettings->ColorOffsetHighlights = Highlights.Offset;

	OutPostProcessSettings->ColorCorrectionShadowsMax = ShadowsMax;
	OutPostProcessSettings->ColorCorrectionHighlightsMin = HighlightsMin;
	OutPostProcessSettings->ColorCorrectionHighlightsMax = HighlightsMax;
}

void FFilmStockSettings::ExportToPostProcessSettings(FPostProcessSettings* OutPostProcessSettings) const
{
	OutPostProcessSettings->bOverride_FilmSlope = true;
	OutPostProcessSettings->bOverride_FilmToe = true;
	OutPostProcessSettings->bOverride_FilmShoulder = true;
	OutPostProcessSettings->bOverride_FilmBlackClip = true;
	OutPostProcessSettings->bOverride_FilmWhiteClip = true;

	OutPostProcessSettings->FilmSlope = Slope;
	OutPostProcessSettings->FilmToe = Toe;
	OutPostProcessSettings->FilmShoulder = Shoulder;
	OutPostProcessSettings->FilmBlackClip = BlackClip;
	OutPostProcessSettings->FilmWhiteClip = WhiteClip;
}

void FGaussianSumBloomSettings::ExportToPostProcessSettings(FPostProcessSettings* OutPostProcessSettings) const
{
	OutPostProcessSettings->bOverride_BloomGaussianIntensity = true;
	OutPostProcessSettings->bOverride_BloomThreshold = true;
	OutPostProcessSettings->bOverride_BloomSizeScale = true;
	OutPostProcessSettings->bOverride_Bloom1Tint = true;
	OutPostProcessSettings->bOverride_Bloom1Size = true;
	OutPostProcessSettings->bOverride_Bloom2Tint = true;
	OutPostProcessSettings->bOverride_Bloom2Size = true;
	OutPostProcessSettings->bOverride_Bloom3Tint = true;
	OutPostProcessSettings->bOverride_Bloom3Size = true;
	OutPostProcessSettings->bOverride_Bloom4Tint = true;
	OutPostProcessSettings->bOverride_Bloom4Size = true;
	OutPostProcessSettings->bOverride_Bloom5Tint = true;
	OutPostProcessSettings->bOverride_Bloom5Size = true;
	OutPostProcessSettings->bOverride_Bloom6Tint = true;
	OutPostProcessSettings->bOverride_Bloom6Size = true;

	OutPostProcessSettings->BloomGaussianIntensity = Intensity;
	OutPostProcessSettings->BloomThreshold = Threshold;
	OutPostProcessSettings->BloomSizeScale = SizeScale;
	OutPostProcessSettings->Bloom1Tint = Filter1Tint;
	OutPostProcessSettings->Bloom1Size = Filter1Size;
	OutPostProcessSettings->Bloom2Tint = Filter2Tint;
	OutPostProcessSettings->Bloom2Size = Filter2Size;
	OutPostProcessSettings->Bloom3Tint = Filter3Tint;
	OutPostProcessSettings->Bloom3Size = Filter3Size;
	OutPostProcessSettings->Bloom4Tint = Filter4Tint;
	OutPostProcessSettings->Bloom4Size = Filter4Size;
	OutPostProcessSettings->Bloom5Tint = Filter5Tint;
	OutPostProcessSettings->Bloom5Size = Filter5Size;
	OutPostProcessSettings->Bloom6Tint = Filter6Tint;
	OutPostProcessSettings->Bloom6Size = Filter6Size;
}

void FConvolutionBloomSettings::ExportToPostProcessSettings(FPostProcessSettings* OutPostProcessSettings) const
{
	OutPostProcessSettings->bOverride_BloomConvolutionIntensity = true;
	OutPostProcessSettings->bOverride_BloomConvolutionTexture = true;
	OutPostProcessSettings->bOverride_BloomConvolutionSize = true;
	OutPostProcessSettings->bOverride_BloomConvolutionScatterDispersion = true;
	OutPostProcessSettings->bOverride_BloomConvolutionCenterUV = true;
	OutPostProcessSettings->bOverride_BloomConvolutionPreFilterMin = true;
	OutPostProcessSettings->bOverride_BloomConvolutionPreFilterMax = true;
	OutPostProcessSettings->bOverride_BloomConvolutionPreFilterMult = true;
	OutPostProcessSettings->bOverride_BloomConvolutionBufferScale = true;

	OutPostProcessSettings->BloomConvolutionIntensity = Intensity;
	OutPostProcessSettings->BloomConvolutionTexture = Texture;
	OutPostProcessSettings->BloomConvolutionSize = Size;
	OutPostProcessSettings->BloomConvolutionScatterDispersion = ScatterDispersion;
	OutPostProcessSettings->BloomConvolutionCenterUV = CenterUV;
	OutPostProcessSettings->BloomConvolutionPreFilterMin = PreFilterMin;
	OutPostProcessSettings->BloomConvolutionPreFilterMax = PreFilterMax;
	OutPostProcessSettings->BloomConvolutionPreFilterMult = PreFilterMult;
	OutPostProcessSettings->BloomConvolutionBufferScale = BufferScale;
}

void FLensBloomSettings::ExportToPostProcessSettings(FPostProcessSettings* OutPostProcessSettings) const
{
	OutPostProcessSettings->bOverride_BloomIntensity = true;
	OutPostProcessSettings->BloomIntensity = Intensity;
	
	GaussianSum.ExportToPostProcessSettings(OutPostProcessSettings);
	Convolution.ExportToPostProcessSettings(OutPostProcessSettings);

	OutPostProcessSettings->bOverride_BloomMethod = true;
	OutPostProcessSettings->BloomMethod = Method;
}

bool FLensBloomSettings::Serialize(FArchive& Ar)
{
	Ar.UsingCustomVersion(FFortniteMainBranchObjectVersion::GUID);
	return false;
}

void FLensBloomSettings::PostSerialize(const FArchive& Ar)
{
	if (Ar.IsLoading())
	{
		if (Ar.CustomVer(FFortniteMainBranchObjectVersion::GUID) < FFortniteMainBranchObjectVersion::SpecializeBloomIntensity)
		{
			Intensity = GaussianSum.Intensity;
			GaussianSum.Intensity = 1.0f;
			Convolution.Intensity = 1.0f;
		}
	}
}

void FLensImperfectionSettings::ExportToPostProcessSettings(FPostProcessSettings* OutPostProcessSettings) const
{
	OutPostProcessSettings->bOverride_BloomDirtMask = true;
	OutPostProcessSettings->bOverride_BloomDirtMaskIntensity = true;
	OutPostProcessSettings->bOverride_BloomDirtMaskTint = true;

	OutPostProcessSettings->BloomDirtMask = DirtMask;
	OutPostProcessSettings->BloomDirtMaskIntensity = DirtMaskIntensity;
	OutPostProcessSettings->BloomDirtMaskTint = DirtMaskTint;
}

void FLensSettings::ExportToPostProcessSettings(FPostProcessSettings* OutPostProcessSettings) const
{
	Bloom.ExportToPostProcessSettings(OutPostProcessSettings);
	Imperfections.ExportToPostProcessSettings(OutPostProcessSettings);

	OutPostProcessSettings->bOverride_SceneFringeIntensity = true;
	OutPostProcessSettings->SceneFringeIntensity = ChromaticAberration;
}

FCameraExposureSettings::FCameraExposureSettings()
{
	static const auto VarDefaultAutoExposureExtendDefaultLuminanceRange = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.DefaultFeature.AutoExposure.ExtendDefaultLuminanceRange"));
	const bool bExtendedLuminanceRange = VarDefaultAutoExposureExtendDefaultLuminanceRange->GetValueOnAnyThread() == 1;

	static const auto VarDefaultAutoExposureBias = IConsoleManager::Get().FindTConsoleVariableDataFloat(TEXT("r.DefaultFeature.AutoExposure.Bias"));
	const float BaseAutoExposureBias = VarDefaultAutoExposureBias->GetValueOnAnyThread();

	// next value might get overwritten by r.DefaultFeature.AutoExposure.Method
	Method = AEM_Histogram;
	LowPercent = 10.0f;
	HighPercent = 90.0f;

	if (bExtendedLuminanceRange)
	{
		// When this project setting is set, the following values are in EV100.
		MinBrightness = -10.0f;
		MaxBrightness = 20.0f;
		HistogramLogMin = -10.0f;
		HistogramLogMax = 20.0f;
	}
	else
	{
		MinBrightness = 0.03f;
		MaxBrightness = 8.0f;
		HistogramLogMin = -8.0f;
		HistogramLogMax = 4.0f;
	}

	SpeedUp = 3.0f;
	SpeedDown = 1.0f;
	Bias = BaseAutoExposureBias;
	CalibrationConstant	= 18.0;

	ApplyPhysicalCameraExposure = 0;
}

void FCameraExposureSettings::ExportToPostProcessSettings(FPostProcessSettings* OutPostProcessSettings) const
{
	OutPostProcessSettings->bOverride_AutoExposureMethod = true;
	OutPostProcessSettings->bOverride_AutoExposureLowPercent = true;
	OutPostProcessSettings->bOverride_AutoExposureHighPercent = true;
	OutPostProcessSettings->bOverride_AutoExposureMinBrightness = true;
	OutPostProcessSettings->bOverride_AutoExposureMaxBrightness = true;
	OutPostProcessSettings->bOverride_AutoExposureSpeedUp = true;
	OutPostProcessSettings->bOverride_AutoExposureSpeedDown = true;
	OutPostProcessSettings->bOverride_AutoExposureBias = true;
	OutPostProcessSettings->bOverride_AutoExposureBiasCurve = true;
	OutPostProcessSettings->bOverride_AutoExposureMeterMask = true;
	OutPostProcessSettings->bOverride_HistogramLogMin = true;
	OutPostProcessSettings->bOverride_HistogramLogMax = true;
	OutPostProcessSettings->bOverride_AutoExposureApplyPhysicalCameraExposure = true;

	OutPostProcessSettings->AutoExposureLowPercent = LowPercent;
	OutPostProcessSettings->AutoExposureHighPercent = HighPercent;
	OutPostProcessSettings->AutoExposureMinBrightness = MinBrightness;
	OutPostProcessSettings->AutoExposureMaxBrightness = MaxBrightness;
	OutPostProcessSettings->AutoExposureSpeedUp = SpeedUp;
	OutPostProcessSettings->AutoExposureSpeedDown = SpeedDown;
	OutPostProcessSettings->AutoExposureBias = Bias;
	OutPostProcessSettings->AutoExposureBiasCurve = BiasCurve;
	OutPostProcessSettings->AutoExposureMeterMask = MeterMask;
	OutPostProcessSettings->HistogramLogMin = HistogramLogMin;
	OutPostProcessSettings->HistogramLogMax = HistogramLogMax;
	OutPostProcessSettings->AutoExposureApplyPhysicalCameraExposure = ApplyPhysicalCameraExposure;
}


// Check there is no divergence between FPostProcessSettings and the smaller settings structures.
#if DO_CHECK && WITH_EDITOR

static void VerifyPostProcessingProperties(
	const FString& PropertyPrefix,
	const TArray<const UStruct*>& NewStructs,
	const TArray<FString>& NewStructPrefixes,
	const TMap<FString, FString>& RenameMap)
{
	check(NewStructs.Num() == NewStructPrefixes.Num());
	const UStruct* LegacyStruct = FPostProcessSettings::StaticStruct();

	TMap<FString, const FProperty*> NewPropertySet;

	// Walk new struct and build list of property name.
	for (int Index = 0; Index < NewStructs.Num(); ++Index)
	{
		const UStruct* NewStruct = NewStructs[Index];
		const FString& NewStructPrefix = NewStructPrefixes[Index];

		for (FProperty* Property = NewStruct->PropertyLink; Property; Property = Property->PropertyLinkNext)
		{
			// Make sure there is no duplicate.
			FString Name = NewStructPrefix + Property->GetNameCPP();
			check(!NewPropertySet.Contains(Name));
			NewPropertySet.Add(Name, Property);
		}
	}

	// Walk FPostProcessSettings.
	for (FProperty* Property = LegacyStruct->PropertyLink; Property; Property = Property->PropertyLinkNext)
	{
		if (!Property->GetNameCPP().StartsWith(PropertyPrefix))
		{
			continue;
		}

		FString OldPropertyName = Property->GetName();
		FString NewPropertyName = OldPropertyName.Mid(PropertyPrefix.Len());

		if (RenameMap.Contains(Property->GetNameCPP()))
		{
			if (RenameMap.FindChecked(Property->GetNameCPP()) == TEXT(""))
			{
				// This property is part of deprecated feature (such as legacy tonemapper).
				check(!NewPropertySet.Contains(NewPropertyName));
				continue;
			}

			NewPropertyName = RenameMap.FindChecked(Property->GetNameCPP());
		}

		if (Property->GetNameCPP().EndsWith(TEXT("_DEPRECATED")))
		{
			check(!NewPropertySet.Contains(NewPropertyName));
		}
		else
		{
			check(Property->SameType(NewPropertySet.FindChecked(NewPropertyName)));
		}
	}
}

static void DoPostProcessSettingsSanityCheck()
{
	{
		TMap<FString, FString> RenameMap;
		RenameMap.Add(TEXT("BloomThreshold"), TEXT("GaussianThreshold"));
		RenameMap.Add(TEXT("BloomSizeScale"), TEXT("GaussianSizeScale"));
		RenameMap.Add(TEXT("Bloom1Size"), TEXT("GaussianFilter1Size"));
		RenameMap.Add(TEXT("Bloom2Size"), TEXT("GaussianFilter2Size"));
		RenameMap.Add(TEXT("Bloom3Size"), TEXT("GaussianFilter3Size"));
		RenameMap.Add(TEXT("Bloom4Size"), TEXT("GaussianFilter4Size"));
		RenameMap.Add(TEXT("Bloom5Size"), TEXT("GaussianFilter5Size"));
		RenameMap.Add(TEXT("Bloom6Size"), TEXT("GaussianFilter6Size"));
		RenameMap.Add(TEXT("Bloom1Tint"), TEXT("GaussianFilter1Tint"));
		RenameMap.Add(TEXT("Bloom2Tint"), TEXT("GaussianFilter2Tint"));
		RenameMap.Add(TEXT("Bloom3Tint"), TEXT("GaussianFilter3Tint"));
		RenameMap.Add(TEXT("Bloom4Tint"), TEXT("GaussianFilter4Tint"));
		RenameMap.Add(TEXT("Bloom5Tint"), TEXT("GaussianFilter5Tint"));
		RenameMap.Add(TEXT("Bloom6Tint"), TEXT("GaussianFilter6Tint"));

		VerifyPostProcessingProperties(TEXT("Bloom"),
			TArray<const UStruct*>({
				FGaussianSumBloomSettings::StaticStruct(),
				FConvolutionBloomSettings::StaticStruct(),
				FLensBloomSettings::StaticStruct(),
				FLensImperfectionSettings::StaticStruct()}),
			TArray<FString>({
				TEXT("Gaussian"),
				TEXT("Convolution"),
				TEXT(""),
				TEXT(""),
			}),
			RenameMap);
	}

	{
		TMap<FString, FString> RenameMap;
		VerifyPostProcessingProperties(TEXT("Exposure"),
			TArray<const UStruct*>({
				FCameraExposureSettings::StaticStruct()}),
			TArray<FString>({
				TEXT(""),
			}),
			RenameMap);
	}

	{
		TMap<FString, FString> RenameMap;
		// Film Grain are ignored
		RenameMap.Add(TEXT("FilmGrainIntensity"), TEXT(""));
		RenameMap.Add(TEXT("FilmGrainIntensityShadows"), TEXT(""));
		RenameMap.Add(TEXT("FilmGrainIntensityMidtones"), TEXT(""));
		RenameMap.Add(TEXT("FilmGrainIntensityHighlights"), TEXT(""));
		RenameMap.Add(TEXT("FilmGrainShadowsMax"), TEXT(""));
		RenameMap.Add(TEXT("FilmGrainHighlightsMin"), TEXT(""));
		RenameMap.Add(TEXT("FilmGrainHighlightsMax"), TEXT(""));
		RenameMap.Add(TEXT("FilmGrainScale"), TEXT(""));
		RenameMap.Add(TEXT("FilmGrainTexture"), TEXT(""));
		VerifyPostProcessingProperties(TEXT("Film"),
			TArray<const UStruct*>({FFilmStockSettings::StaticStruct()}),
			TArray<FString>({
				TEXT(""),
			}),
			RenameMap);
	}
}

#endif // DO_CHECK

FPostProcessSettings::FPostProcessSettings()
{
	// to set all bOverride_.. by default to false
	FMemory::Memzero(this, sizeof(FPostProcessSettings));

	TemperatureType = ETemperatureMethod::TEMP_WhiteBalance;
	WhiteTemp = 6500.0f;
	WhiteTint = 0.0f;

	// Color Correction controls
	ColorSaturation = FVector4(1.0f, 1.0f, 1.0f, 1.0f);
	ColorContrast = FVector4(1.0f, 1.0f, 1.0f, 1.0f);
	ColorGamma = FVector4(1.0f, 1.0f, 1.0f, 1.0f);
	ColorGain = FVector4(1.0f, 1.0f, 1.0f, 1.0f);
	ColorOffset = FVector4(0.0f, 0.0f, 0.0f, 0.0f);

	ColorSaturationShadows = FVector4(1.0f, 1.0f, 1.0f, 1.0f);
	ColorContrastShadows = FVector4(1.0f, 1.0f, 1.0f, 1.0f);
	ColorGammaShadows = FVector4(1.0f, 1.0f, 1.0f, 1.0f);
	ColorGainShadows = FVector4(1.0f, 1.0f, 1.0f, 1.0f);
	ColorOffsetShadows = FVector4(0.0f, 0.0f, 0.0f, 0.0f);

	ColorSaturationMidtones = FVector4(1.0f, 1.0f, 1.0f, 1.0f);
	ColorContrastMidtones = FVector4(1.0f, 1.0f, 1.0f, 1.0f);
	ColorGammaMidtones = FVector4(1.0f, 1.0f, 1.0f, 1.0f);
	ColorGainMidtones = FVector4(1.0f, 1.0f, 1.0f, 1.0f);
	ColorOffsetMidtones = FVector4(0.f, 0.0f, 0.0f, 0.0f);

	ColorSaturationHighlights = FVector4(1.0f, 1.0f, 1.0f, 1.0f);
	ColorContrastHighlights = FVector4(1.0f, 1.0f, 1.0f, 1.0f);
	ColorGammaHighlights = FVector4(1.0f, 1.0f, 1.0f, 1.0f);
	ColorGainHighlights = FVector4(1.0f, 1.0f, 1.0f, 1.0f);
	ColorOffsetHighlights = FVector4(0.0f, 0.0f, 0.0f, 0.0f);

	ColorCorrectionShadowsMax = 0.09f;
	ColorCorrectionHighlightsMin = 0.5f;
	ColorCorrectionHighlightsMax = 1.0f;

	BlueCorrection = 0.6f;
	ExpandGamut = 1.0f;
	ToneCurveAmount = 1.0;

	TonemappingMethod = ETonemappingMethod::Filmic;

	// default values:

	// ACES settings
	FilmSlope = 0.88f;
	FilmToe = 0.55f;
	FilmShoulder = 0.26f;
	FilmBlackClip = 0.0f;
	FilmWhiteClip = 0.04f;

	SceneColorTint = FLinearColor(1, 1, 1);
	SceneFringeIntensity = 0.0f;
	BloomMethod = BM_SOG;
	// next value might get overwritten by r.DefaultFeature.Bloom
	BloomIntensity = 0.675f;
	BloomGaussianIntensity = 1.0f;
	BloomThreshold = -1.0f;
	// default is 4 to maintain old settings after fixing something that caused a factor of 4
	BloomSizeScale = 4.0;
	Bloom1Tint = FLinearColor(0.3465f, 0.3465f, 0.3465f);
	Bloom1Size = 0.3f;
	Bloom2Tint = FLinearColor(0.138f, 0.138f, 0.138f);
	Bloom2Size = 1.0f;
	Bloom3Tint = FLinearColor(0.1176f, 0.1176f, 0.1176f);
	Bloom3Size = 2.0f;
	Bloom4Tint = FLinearColor(0.066f, 0.066f, 0.066f);
	Bloom4Size = 10.0f;
	Bloom5Tint = FLinearColor(0.066f, 0.066f, 0.066f);
	Bloom5Size = 30.0f;
	Bloom6Tint = FLinearColor(0.061f, 0.061f, 0.061f);
	Bloom6Size = 64.0f;
	BloomConvolutionIntensity = 1.0f;
	BloomConvolutionScatterDispersion = 1.f;
	BloomConvolutionSize = 1.f;
	BloomConvolutionCenterUV = FVector2D(0.5f, 0.5f);
#if WITH_EDITORONLY_DATA
	BloomConvolutionPreFilter_DEPRECATED = FVector3f(-1.f, -1.f, -1.f);
	DepthOfFieldMethod_DEPRECATED = EDepthOfFieldMethod::DOFM_MAX;
#endif
	BloomConvolutionPreFilterMin = 7.f;
	BloomConvolutionPreFilterMax = 15000.f;
	BloomConvolutionPreFilterMult = 15.f;
	BloomConvolutionBufferScale = 0.133f;
	BloomDirtMaskIntensity = 0.0f;
	BloomDirtMaskTint = FLinearColor(0.5f, 0.5f, 0.5f);
	bMegaLights = true;
	MegaLightsFrontLayerTranslucency = false;
	AmbientCubemapIntensity = 1.0f;
	AmbientCubemapTint = FLinearColor(1, 1, 1);
	CameraShutterSpeed = 60.f;
	CameraISO = 100.f;
	AutoExposureCalibrationConstant_DEPRECATED = 16.f;
	// next value might get overwritten by r.DefaultFeature.AutoExposure.Method
	AutoExposureMethod = AEM_Histogram;
	AutoExposureLowPercent = 10.0f;
	AutoExposureHighPercent = 90.0f;

	// next value might get overwritten by r.DefaultFeature.AutoExposure
	static const auto VarDefaultAutoExposureExtendDefaultLuminanceRange = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.DefaultFeature.AutoExposure.ExtendDefaultLuminanceRange"));
	if (VarDefaultAutoExposureExtendDefaultLuminanceRange->GetValueOnAnyThread() != 0)
	{
		// When this project setting is set, the following values are in EV100.
		AutoExposureMinBrightness = -10.0f;
		AutoExposureMaxBrightness = 20.0f;
		HistogramLogMin = -10.0f;
		HistogramLogMax = 20.0f;
	}
	else
	{
		AutoExposureMinBrightness = 0.03f;
		AutoExposureMaxBrightness = 8.0f;
		HistogramLogMin = -8.0f;
		HistogramLogMax = 4.0f;
	}

	static const auto VarDefaultAutoExposureBias = IConsoleManager::Get().FindTConsoleVariableDataFloat(TEXT("r.DefaultFeature.AutoExposure.Bias"));
	const float BaseAutoExposureBias = VarDefaultAutoExposureBias->GetValueOnAnyThread();

	AutoExposureBias = BaseAutoExposureBias;
	AutoExposureSpeedUp = 3.0f;
	AutoExposureSpeedDown = 1.0f;

	AutoExposureApplyPhysicalCameraExposure = 1;

	LocalExposureMethod = ELocalExposureMethod::Bilateral;
	LocalExposureContrastScale_DEPRECATED = 1.0f;
	LocalExposureHighlightContrastScale = 1.0f;
	LocalExposureShadowContrastScale = 1.0f;
	LocalExposureDetailStrength = 1.0f;
	LocalExposureBlurredLuminanceBlend = 0.6f;
	LocalExposureBlurredLuminanceKernelSizePercent = 50.0f;
	LocalExposureHighlightThresholdStrength = 1.0f;
	LocalExposureShadowThresholdStrength = 1.0f;
	LocalExposureMiddleGreyBias = 0.0f;

	// next value might get overwritten by r.DefaultFeature.LensFlare
	LensFlareIntensity = 1.0f;
	LensFlareTint = FLinearColor(1.0f, 1.0f, 1.0f);
	LensFlareBokehSize = 3.0f;
	LensFlareThreshold = 8.0f;
	VignetteIntensity = 0.4f;
	VignetteType = EVignetteType::CosineFourthLaw;
	VignetteCenter = FVector2f(0.5f, 0.5f);
	VignetteColor = FLinearColor::Black;
	VignetteSize = FVector2f(1.0f, 1.0f);
	VignetteSoftness = 5.0f;
	Sharpen = 0.0f;
	GrainIntensity_DEPRECATED = 0.0f;
	GrainJitter_DEPRECATED = 0.0f;

	// Film Grain
	FilmGrainIntensity = 0.0f;
	FilmGrainIntensityShadows = 1.0f;
	FilmGrainIntensityMidtones = 1.0f;
	FilmGrainIntensityHighlights = 1.0f;
	FilmGrainShadowsMax = 0.09f;
	FilmGrainHighlightsMin = 0.5f;
	FilmGrainHighlightsMax = 1.0f;
#if WITH_EDITORONLY_DATA
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FilmGrainTexelSize_DEPRECATED = 1.0f;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
#endif
	FilmGrainScale = FVector2f{1.0f, 1.0f};

	// next value might get overwritten by r.DefaultFeature.AmbientOcclusion
	AmbientOcclusionIntensity = .5f;
	// next value might get overwritten by r.DefaultFeature.AmbientOcclusionStaticFraction
	AmbientOcclusionStaticFraction = 1.0f;
	AmbientOcclusionRadius = 200.0f;
	AmbientOcclusionDistance_DEPRECATED = 80.0f;
	AmbientOcclusionFadeDistance = 8000.0f;
	AmbientOcclusionFadeRadius = 5000.0f;
	AmbientOcclusionPower = 2.0f;
	AmbientOcclusionBias = 3.0f;
	AmbientOcclusionQuality = 50.0f;
	AmbientOcclusionMipBlend = 0.6f;
	AmbientOcclusionMipScale = 1.7f;
	AmbientOcclusionMipThreshold = 0.01f;
	AmbientOcclusionRadiusInWS = false;
	AmbientOcclusionTemporalBlendWeight = 0.1f;
	RayTracingAO = 0;
	RayTracingAOSamplesPerPixel = 1;
	RayTracingAOIntensity = 1.0;
	RayTracingAORadius = 200.0f;
	DynamicGlobalIlluminationMethod = EDynamicGlobalIlluminationMethod::Lumen;
	IndirectLightingColor = FLinearColor(1.0f, 1.0f, 1.0f);
	IndirectLightingIntensity = 1.0f;
	LumenSurfaceCacheResolution = 1.0f;
	LumenSceneLightingQuality = 1;
	LumenSceneDetail = 1.0f;
	LumenSceneViewDistance = 20000.0f;
	LumenSceneLightingUpdateSpeed = 1;
	LumenFinalGatherQuality = 1;
	LumenFinalGatherLightingUpdateSpeed = 1;
	LumenFinalGatherScreenTraces = 1;
	LumenMaxTraceDistance = 20000.0f;
	LumenDiffuseColorBoost = 1.0f;
	LumenAmbientOcclusionIntensity = 1.0f;
	LumenSkylightLeaking = 0.0f;
	LumenSkylightLeakingTint = FLinearColor(1.0f, 1.0f, 1.0f, 1.0f);
	LumenFullSkylightLeakingDistance = 1000.0f;

	ColorGradingIntensity = 1.0f;

	DepthOfFieldFocalDistance = 0; // Intentionally invalid to disable DOF by default.
	DepthOfFieldFstop = 4.0f; 
	DepthOfFieldMinFstop = 1.2f;
	DepthOfFieldBladeCount = FPostProcessSettings::kDefaultDepthOfFieldBladeCount;
	DepthOfFieldEnabled = true;
	DepthOfFieldSensorWidth = 24.576f;			// APS-C
	DepthOfFieldSqueezeFactor = 1.0f;
	DepthOfFieldDepthBlurAmount = 1.0f;
	DepthOfFieldDepthBlurRadius = 0.0f;
	DepthOfFieldUseHairDepth = 0;
	DepthOfFieldPetzvalBokeh = 0;
	DepthOfFieldPetzvalBokehFalloff = 1.0f;
	DepthOfFieldPetzvalExclusionBoxExtents = FVector2f::ZeroVector;
	DepthOfFieldPetzvalExclusionBoxRadius = 0.0f;
	DepthOfFieldAspectRatioScalar = 1.0f;
	DepthOfFieldBarrelRadius = 5;
	DepthOfFieldBarrelLength = 0;
	for (int Index = 0; Index < UE_ARRAY_COUNT(DepthOfFieldMatteBoxFlags); ++Index)
	{
		DepthOfFieldMatteBoxFlags[Index] = {};
	}
	DepthOfFieldFocalRegion = 0.0f;
	DepthOfFieldNearTransitionRegion = 300.0f;
	DepthOfFieldFarTransitionRegion = 500.0f;
	DepthOfFieldScale = 0.0f;
	DepthOfFieldNearBlurSize = 15.0f;
	DepthOfFieldFarBlurSize = 15.0f;
	DepthOfFieldOcclusion = 0.4f;
	DepthOfFieldSkyFocusDistance = 0.0f;
	// 200 should be enough even for extreme aspect ratios to give the default no effect
	DepthOfFieldVignetteSize = 200.0f;
	LensFlareTints[0] = FLinearColor(1.0f, 0.8f, 0.4f, 0.6f);
	LensFlareTints[1] = FLinearColor(1.0f, 1.0f, 0.6f, 0.53f);
	LensFlareTints[2] = FLinearColor(0.8f, 0.8f, 1.0f, 0.46f);
	LensFlareTints[3] = FLinearColor(0.5f, 1.0f, 0.4f, 0.39f);
	LensFlareTints[4] = FLinearColor(0.5f, 0.8f, 1.0f, 0.31f);
	LensFlareTints[5] = FLinearColor(0.9f, 1.0f, 0.8f, 0.27f);
	LensFlareTints[6] = FLinearColor(1.0f, 0.8f, 0.4f, 0.22f);
	LensFlareTints[7] = FLinearColor(0.9f, 0.7f, 0.7f, 0.15f);
	// next value might get overwritten by r.DefaultFeature.MotionBlur
	MotionBlurAmount = 0.5f;
	MotionBlurMax = 5.0f;
	MotionBlurTargetFPS = 30;
	MotionBlurPerObjectSize = 0.f;
	ScreenPercentage_DEPRECATED = 100.0f;
	ReflectionsType_DEPRECATED = EReflectionsType::RayTracing;

	ReflectionMethod = EReflectionMethod::Lumen;
	LumenReflectionQuality = 1;
	LumenRayLightingMode = ELumenRayLightingModeOverride::Default;
	LumenReflectionsScreenTraces = 1;
	LumenFrontLayerTranslucencyReflections = false;
	LumenMaxRoughnessToTraceReflections = 0.4f;
	LumenMaxReflectionBounces = 1;

	LumenMaxRefractionBounces = 0;

	ScreenSpaceReflectionIntensity = 100.0f;
	ScreenSpaceReflectionQuality = 50.0f;
	ScreenSpaceReflectionMaxRoughness = 0.6f;

	TranslucencyType = ETranslucencyType::Raster;
	RayTracingTranslucencyMaxRoughness = 0.6f;
	RayTracingTranslucencyRefractionRays = 3; // 3 to: first hit surface, second hit back inner surface and a third to fetch the background.
	RayTracingTranslucencySamplesPerPixel = 1;
	RayTracingTranslucencyShadows = EReflectedAndRefractedRayTracedShadows::Hard_shadows;
	RayTracingTranslucencyRefraction = 1;
	RayTracingTranslucencyMaxPrimaryHitEvents = 4;
	RayTracingTranslucencyMaxSecondaryHitEvents = 2;
	RayTracingTranslucencyUseRayTracedRefraction = 0;

	PathTracingMaxBounces = 32;
	PathTracingSamplesPerPixel = 2048;
	PathTracingMaxPathIntensity = 24.0f;
	PathTracingEnableEmissiveMaterials = 1;
	PathTracingEnableReferenceDOF = 0;
	PathTracingEnableReferenceAtmosphere = 0;
	PathTracingEnableDenoiser = 1;

	PathTracingIncludeEmissive = 1;
	PathTracingIncludeDiffuse = 1;
	PathTracingIncludeIndirectDiffuse = 1;
	PathTracingIncludeSpecular = 1;
	PathTracingIncludeIndirectSpecular = 1;
	PathTracingIncludeVolume = 1;
	PathTracingIncludeIndirectVolume = 1;

	bMobileHQGaussian = false;

	UserFlags = 0;

#if DO_CHECK && WITH_EDITOR
	static bool bCheckedMembers = false;
	if (!bCheckedMembers)
	{
		bCheckedMembers = true;
		DoPostProcessSettingsSanityCheck();
	}
#endif // DO_CHECK
}


#if WITH_EDITORONLY_DATA
bool FPostProcessSettings::Serialize(FArchive& Ar)
{
	Ar.UsingCustomVersion(FRenderingObjectVersion::GUID);
	Ar.UsingCustomVersion(FReleaseObjectVersion::GUID);
	Ar.UsingCustomVersion(FUE5ReleaseStreamObjectVersion::GUID);

	// Don't actually serialize, just write the custom version for PostSerialize
	return false;
}

// Conversion estimate for auto-exposure parameters from 4.24 to 4.25. Because we reduced the complexity of the algorithm,
// we can't do an exact match of legacy parameters to current parameters. So we are storing the backup exposure compensation just in case.
float CalculateEyeAdaptationExposureVersionUpdate(const FPostProcessSettings& Settings, const bool bExtendedLuminanceRange)
{
	float ExtraExposureCompensation = 0.0f;

	// Since we are doing this without CVars, we can not call LuminanceMaxFromLensAttenuation(). Instead, assume 1.0
	const float CurrLuminanceMax = 1.0f; 
	const float PrevLuminanceMax = bExtendedLuminanceRange ? 1.2f : 1.0f;

	// In legacy, some parameters (like Calibration Constant) would apply an exposure bias before the min/max luminance clamp. So if we
	// match the previous look of Calibration Constant by adding an exposure bias, it will look fine with wide EV ranges.  But it is 
	// quite common for levels to fake manual exposure by forcing min and max luminance to the same value. In those cases adding an EV
	// bias causes more problems than it solves.
	//
	// Additionally, when bExtendedLuminanceRange is false, in general, the range is pretty tight. So if we add one or two stops of
	// exposure compensation it will often look much too bright.
	//
	// So the compromise is:
	// 1. We always include the Luminance Max 1.2 conversion for Extended range, and keep it at 1.0 for not extended range.
	// 2. When AutoExposureMinBrightness=AutoExposureMaxBrightness, do nothing (fake manual exposure).
	// 3a. In Basic mode, apply the basic Calibration Constant (it behaves slightly differently when bExtendedLuminanceRange is true)
	// 3b. In Histogram mode, when bExtendedLuminanceRange is false, do nothing.
	// 3c. In Histogram mode, otherwise apply a rough perceptual conversion.
	if (Settings.AutoExposureMinBrightness >= Settings.AutoExposureMaxBrightness)
	{
		// fake manual exposure, no op
	}
	else
	{
		if (Settings.AutoExposureMethod == EAutoExposureMethod::AEM_Basic)
		{
			if (bExtendedLuminanceRange)
			{
				// add the Calibration Constant into EC
				ExtraExposureCompensation += FMath::Log2(18.0f / Settings.AutoExposureCalibrationConstant_DEPRECATED);
			}
			else
			{
				// add the Calibration Constant into EC, but it without extended luminance range the minimum is 16.
				ExtraExposureCompensation += FMath::Log2(18 / FMath::Max(16.0f, Settings.AutoExposureCalibrationConstant_DEPRECATED));
			}
		}
		else if (Settings.AutoExposureMethod == EAutoExposureMethod::AEM_Histogram)
		{
			if (bExtendedLuminanceRange)
			{
				// We're matching the look of the old and new formula by adding exposure compensation. But this does not work correctly
				// at the min and max values of the exposure range, because adding exposure compensation pushes it outside the clamped
				// value. So in the general case of AutoExposureMinBrightness < AutoExposureMaxBrightness, tweak the exposure compensation
				// to adjust the look. But in the other special case, do nothing.
				if (Settings.AutoExposureMinBrightness < Settings.AutoExposureMaxBrightness)
				{
					// The previous version of the histogram would take the averge luminance between high and low, and then make that
					// value the white point. This is different from basic exposure (and the new version) which makes that point 
					// middle grey instead. The following formula gives a similar look for auto-conversion.
					const float Average = ((Settings.AutoExposureLowPercent + Settings.AutoExposureHighPercent) * .5f);
					const float X0 = 50.00f;
					const float X1 = 89.15f;
					const float Y0 = 1.0f;
					const float Y1 = 1.5f;

					// These numbers actually under-correct a bit. Since we are doing a hidden upgrade, it's better to undercorrect
					// than to overcorrect.

					const float T = (Average - X0) / (X1 - X0);
					ExtraExposureCompensation += FMath::Lerp(Y0, Y1, T);
				}
				else
				{
					// no op, histogram auto-exposure is effectively being used as manual exposure
				}
			}
			else
			{
				// If we are not in extended luminance range, then do nothing. The default ranges is pretty tight (0.03 linear to 2.0),
				// so if we apply an exposure bias to "fix the look" it will probably do more harm than good.
			}
		}
		else // ConvertedSettings.AutoExposureMethod == EAutoExposureMethod::AEM_Manual
		{
			// nothing to do
		}
	}

	// add the exposure compensation from the LuminanceMax. In general, if extended luminance range is enabled, the old value should be 1.2 and the new
	// value should be 1.0. Otherwise they should both be 1.0.
	ExtraExposureCompensation += FMath::Log2(PrevLuminanceMax/ CurrLuminanceMax);

	return ExtraExposureCompensation;
}

void FPostProcessSettings::PostSerialize(const FArchive& Ar)
{
	if (Ar.IsLoading())
	{
		const int32 RenderingObjectVersion = Ar.CustomVer(FRenderingObjectVersion::GUID);
		const int32 ReleaseObjectVersion = Ar.CustomVer(FReleaseObjectVersion::GUID);

		if (RenderingObjectVersion < FRenderingObjectVersion::DiaphragmDOFOnlyForDeferredShadingRenderer)
		{
			// This is impossible because FocalDistanceDisablesDOF was in Release 4.23 branch where DiaphragmDOFOnlyForDeferredShadingRenderer was already existing.
			check(ReleaseObjectVersion < FReleaseObjectVersion::FocalDistanceDisablesDOF);

			// Make sure the DOF of the deferred shading renderer is enabled if the circle DOF method was used before with previous default setting for DepthOfFieldFstop.
			if (DepthOfFieldFocalDistance == 0.0f && DepthOfFieldMethod_DEPRECATED == DOFM_CircleDOF)
			{
				DepthOfFieldFocalDistance = 1000.0f;
			}
			else if (DepthOfFieldMethod_DEPRECATED != DOFM_CircleDOF)
			{
				// Aggressively force disable DOF by setting default value on the focal distance to be invalid if the method was not CircleDOF, in case.
				// it focal distance was modified if even if DOF was in the end disabled.
				DepthOfFieldFocalDistance = 0.0f;
			}

			// Make sure gaussian DOF is disabled on mobile if the DOF method was set to something else.
			if (DepthOfFieldMethod_DEPRECATED != DOFM_Gaussian)
			{
				DepthOfFieldScale = 0.0f;
			}
		}
		else if (ReleaseObjectVersion < FReleaseObjectVersion::FocalDistanceDisablesDOF)
		{
			// This is only for assets saved in the the window DiaphragmDOFOnlyForDeferredShadingRenderer -> FocalDistanceDisablesDOF
			DepthOfFieldFocalDistance = 0.0f;
		}
		
		if (RenderingObjectVersion < FRenderingObjectVersion::AutoExposureChanges)
		{
			// Backup the exposure bias in case we run into a major problem with the conversion below
			AutoExposureBiasBackup = AutoExposureBias;

			// Only adjust this post process volume if the autoexposure values are selected. Speed up/down and new values are skipped from this check.
			const bool bIsAnyNonDefault = bOverride_AutoExposureBias ||
				bOverride_AutoExposureLowPercent ||
				bOverride_AutoExposureHighPercent ||
				bOverride_AutoExposureBiasCurve ||
				bOverride_AutoExposureMethod ||
				bOverride_AutoExposureMinBrightness ||
				bOverride_AutoExposureMaxBrightness;

			// Calculate an exposure bias to try and keep the look similar from 4.24 to 4.25
			if (bIsAnyNonDefault)
			{
				static const auto VarDefaultAutoExposureExtendDefaultLuminanceRange = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.DefaultFeature.AutoExposure.ExtendDefaultLuminanceRange"));
				bool bExtendedLuminanceRange = (VarDefaultAutoExposureExtendDefaultLuminanceRange->GetValueOnAnyThread() == 1);

				const float ExtraAutoExposureBias = CalculateEyeAdaptationExposureVersionUpdate(*this, bExtendedLuminanceRange);

				AutoExposureBias += ExtraAutoExposureBias;
			}
		}

		if (RenderingObjectVersion < FRenderingObjectVersion::AutoExposureForceOverrideBiasFlag)
		{
			// Backup the exposure bias override in case we run into a major problem
			bOverride_AutoExposureBiasBackup = bOverride_AutoExposureBias;

			// Same logic as FRenderingObjectVersion::AutoExposureChanges
			const bool bIsAnyNonDefault = bOverride_AutoExposureBias ||
				bOverride_AutoExposureLowPercent ||
				bOverride_AutoExposureHighPercent ||
				bOverride_AutoExposureBiasCurve ||
				bOverride_AutoExposureMethod ||
				bOverride_AutoExposureMinBrightness ||
				bOverride_AutoExposureMaxBrightness;

			// Since the auto-exposure was updated in the revision from FRenderingObjectVersion::AutoExposureChanges, we should also
			// override the default value.
			if (bIsAnyNonDefault)
			{
				bOverride_AutoExposureBias = true;
			}
		}

		if (RenderingObjectVersion < FRenderingObjectVersion::AutoExposureDefaultFix)
		{
			// This fix is for a specific corner case of serialization. In between the versions of AutoExposureChanges and
			// AutoExposureForceOverrideBiasFlag, we changed the default AutoExposure bias from 0.0 to 1.0. Which sounds
			// like a harmless change, but actually caused a problematic corner case.
			//
			// A common approach for artists is to use auto-exposure in Histogram mode, with the AutoExposureMinBrightness==AutoExposureMaxBrightness
			// I.e. fake Manual exposure. The version AutoExposureChanges would detect this and set extra exposure compensation to 0.0. But the default
			// has changed which will cause that extra default to be added in. The fix is to zero out AutoExposureBias if our backup value
			// is either 0.0 or the new default parameter.
			//
			// Note that there are some potential, if unlikely side effects. For example, if you have Min=Max brightness, and you intentionally
			// set AutoExposureBias to 1.0 (the new default), this code will revert AutoExposureBias to 1.0.

			// Using the same check as above.
			const bool bIsAnyNonDefault = bOverride_AutoExposureBias ||
				bOverride_AutoExposureLowPercent ||
				bOverride_AutoExposureHighPercent ||
				bOverride_AutoExposureBiasCurve ||
				bOverride_AutoExposureMethod ||
				bOverride_AutoExposureMinBrightness ||
				bOverride_AutoExposureMaxBrightness;

			if (bIsAnyNonDefault)
			{
				static const auto VarDefaultAutoExposureExtendDefaultLuminanceRange = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.DefaultFeature.AutoExposure.ExtendDefaultLuminanceRange"));
				bool bExtendedLuminanceRange = (VarDefaultAutoExposureExtendDefaultLuminanceRange->GetValueOnAnyThread() == 1);

				const float ExtraAutoExposureBias = CalculateEyeAdaptationExposureVersionUpdate(*this, bExtendedLuminanceRange);

				static const auto VarDefaultAutoExposureBias = IConsoleManager::Get().FindTConsoleVariableDataFloat(TEXT("r.DefaultFeature.AutoExposure.Bias"));
				const float BaseAutoExposureBias = VarDefaultAutoExposureBias->GetValueOnAnyThread();

				// We take this path if:
				// 1. We did not originally have bOverride_AutoExposureBias
				// 2. We now do have bOverride_AutoExposureBias (it changed during FRenderingObjectVersion::AutoExposureForceOverrideBiasFlag)
				// 3. Our PPV had the serialized AutoExposureBias of 0.0, which would cause the default to change. Or the previous auto exposure
				//        is the new default, which is also possible.
				// 4. AutoExposureBias is now nonzero.
				if (!bOverride_AutoExposureBiasBackup &&
					bOverride_AutoExposureBias &&
					(AutoExposureBiasBackup == 0.0f || AutoExposureBiasBackup == BaseAutoExposureBias) &&
					AutoExposureBias != 0.0f)
				{
					// Assume previous exposure was 0.0, so ignore AutoExposureBiasBackup and only add the extra bias.
					AutoExposureBias = ExtraAutoExposureBias;
				}
			}
		}

		if (Ar.CustomVer(FUE5ReleaseStreamObjectVersion::GUID) < FUE5ReleaseStreamObjectVersion::ReflectionMethodEnum)
		{
			if (bOverride_ReflectionsType_DEPRECATED)
			{
				bOverride_ReflectionMethod = true;

				if (ReflectionsType_DEPRECATED == EReflectionsType::ScreenSpace)
				{
					ReflectionMethod = EReflectionMethod::ScreenSpace;
				}
			}
		}

		// Before, the convolution bloom would ignore the bloom intensity, but activate when BloomIntensity > 0.0. UE 5.0 changed so that the 
		// BloomIntensity also controls the scatter dispersion of the convolution bloom.
		if (Ar.CustomVer(FUE5ReleaseStreamObjectVersion::GUID) < FUE5ReleaseStreamObjectVersion::ConvolutionBloomIntensity)
		{
			if (BloomMethod == BM_FFT && BloomIntensity > 0.0)
			{
				BloomIntensity = 1.0f;
			}
		}

		// Changed HitLighting to HitLightingForReflections, and HitLighting now means hit lighting for entire Lumen
		if (Ar.CustomVer(FFortniteMainBranchObjectVersion::GUID) < FFortniteMainBranchObjectVersion::LumenRayLightingModeOverrideEnum)
		{
			if (bOverride_LumenRayLightingMode)
			{
				if (LumenRayLightingMode == ELumenRayLightingModeOverride::HitLighting)
				{
					LumenRayLightingMode = ELumenRayLightingModeOverride::HitLightingForReflections;
				}
			}
		}
	}
}
#endif

const FPostProcessSettings& FPostProcessSettings::GetDefault()
{
	static const FPostProcessSettings DefaultSettings;

	return DefaultSettings;
}

UScene::UScene(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}
