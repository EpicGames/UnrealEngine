// Copyright Epic Games, Inc. All Rights Reserved.

#include "SceneInterface.h"
#include "RenderGraphBuilder.h"
#include "SceneTypes.h"
#include "SceneUtils.h"

static thread_local FSceneInterface::IPrimitiveTransformUpdater* PrimitiveTransformUpdaterInstanceTLS = nullptr;

FSceneInterface::IPrimitiveTransformUpdater* FSceneInterface::IPrimitiveTransformUpdater::SetInstanceTLS(IPrimitiveTransformUpdater* Instance)
{
	IPrimitiveTransformUpdater* PreviousInstance = PrimitiveTransformUpdaterInstanceTLS;
	PrimitiveTransformUpdaterInstanceTLS = Instance;
	return PreviousInstance;
}

FSceneInterface::IPrimitiveTransformUpdater* FSceneInterface::IPrimitiveTransformUpdater::GetInstanceTLS()
{
	return PrimitiveTransformUpdaterInstanceTLS;
}

FSceneInterface::FSceneInterface(ERHIFeatureLevel::Type InFeatureLevel)
	: FeatureLevel(InFeatureLevel)
	, ShaderPlatform(GShaderPlatformForFeatureLevel[InFeatureLevel])
{
}

void FSceneInterface::UpdateAllPrimitiveSceneInfos(FRHICommandListImmediate& RHICmdList)
{
	FRDGBuilder GraphBuilder(RHICmdList, FRDGEventName(TEXT("UpdateAllPrimitiveSceneInfos")));
	UpdateAllPrimitiveSceneInfos(GraphBuilder);
	GraphBuilder.Execute();
}


EShadingPath FSceneInterface::GetShadingPath(ERHIFeatureLevel::Type InFeatureLevel)
{
	return GetFeatureLevelShadingPath(InFeatureLevel);
}
