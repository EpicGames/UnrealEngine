// Copyright Epic Games, Inc. All Rights Reserved.

#include "SceneManagement.h"
#include "DeviceProfiles/DeviceProfile.h"
#include "DeviceProfiles/DeviceProfileManager.h"
#include "EngineModule.h"
#include "MaterialShared.h"
#include "PrimitiveSceneProxy.h"
#include "StaticMeshResources.h"
#include "RHIStaticStates.h"
#include "SceneView.h"
#include "Async/ParallelFor.h"
#include "LightMap.h"
#include "LightSceneProxy.h"
#include "ShadowMap.h"
#include "RayTracingInstance.h"
#include "Materials/MaterialRenderProxy.h"
#include "TextureResource.h"
#include "VT/LightmapVirtualTexture.h"
#include "UnrealEngine.h"
#include "ColorManagement/ColorSpace.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "StaticMeshBatch.h"
#include "PrimitiveUniformShaderParametersBuilder.h"
#include "PrimitiveSceneShaderData.h"
#include "RenderGraphBuilder.h"

static TAutoConsoleVariable<float> CVarLODTemporalLag(
	TEXT("lod.TemporalLag"),
	0.5f,
	TEXT("This controls the the time lag for temporal LOD, in seconds."),
	ECVF_Scalability | ECVF_Default);

static float GDebugLODDitherSpeed = -1.0f;
static FAutoConsoleVariableRef CVarForceLODDitherSpeed(
	TEXT("lod.DebugLODDitherSpeed"),
	GDebugLODDitherSpeed,
	TEXT("Force a LOD Dither transition with the next LOD, and have it looping. Useful to debug LOD dithering without moving the camera."),
	ECVF_Cheat | ECVF_RenderThreadSafe);

bool AreCompressedTransformsSupported()
{
	return FDataDrivenShaderPlatformInfo::GetSupportSceneDataCompressedTransforms(GMaxRHIShaderPlatform);
}

bool DoesPlatformSupportDistanceFields(const FStaticShaderPlatform Platform)
{
	return FDataDrivenShaderPlatformInfo::GetSupportsDistanceFields(Platform);
}

bool DoesPlatformSupportDistanceFieldShadowing(EShaderPlatform Platform)
{
	return DoesPlatformSupportDistanceFields(Platform);
}

bool DoesPlatformSupportDistanceFieldAO(EShaderPlatform Platform)
{
	return DoesPlatformSupportDistanceFields(Platform) && (!IsMobilePlatform(Platform) || IsMobileDistanceFieldAOEnabled(Platform));
}

bool DoesProjectSupportDistanceFields()
{
	static const auto CVarGenerateDF = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.GenerateMeshDistanceFields"));
	static const auto CVarDFIfNoHWRT = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.DistanceFields.SupportEvenIfHardwareRayTracingSupported"));

	return DoesPlatformSupportDistanceFields(GMaxRHIShaderPlatform)
		&& CVarGenerateDF->GetValueOnAnyThread() != 0
		&& (CVarDFIfNoHWRT->GetValueOnAnyThread() != 0 || !IsRayTracingAllowed());
}

bool ShouldAllPrimitivesHaveDistanceField(EShaderPlatform ShaderPlatform)
{
	return (DoesPlatformSupportDistanceFieldAO(ShaderPlatform) || DoesPlatformSupportDistanceFieldShadowing(ShaderPlatform))
		&& IsUsingDistanceFields(ShaderPlatform)
		&& DoesProjectSupportDistanceFields();
}

bool ShouldCompileDistanceFieldShaders(EShaderPlatform ShaderPlatform)
{
	static const auto CVarGenerateDF = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.GenerateMeshDistanceFields"));

	return DoesPlatformSupportDistanceFields(ShaderPlatform) 
		&& IsUsingDistanceFields(ShaderPlatform) 
		&& CVarGenerateDF->GetValueOnAnyThread() != 0;
}

bool ShouldCompileDistanceFieldAOShaders(EShaderPlatform ShaderPlatform)
{
	return ShouldCompileDistanceFieldShaders(ShaderPlatform) 
		&& DoesPlatformSupportDistanceFieldAO(ShaderPlatform);
}

float FTemporalLODState::GetTemporalLODTransition(float LastRenderTime) const
{
	if (GDebugLODDitherSpeed > 0.0f)
	{
		return FMath::Frac(LastRenderTime * GDebugLODDitherSpeed);
	}

	if (TemporalLODLag == 0.0)
	{
		return 0.0f; // no fade
	}

	return FMath::Clamp((LastRenderTime - TemporalLODLag - TemporalLODTime[0]) / (TemporalLODTime[1] - TemporalLODTime[0]), 0.0f, 1.0f);
}

void FTemporalLODState::UpdateTemporalLODTransition(const FSceneView& View, float LastRenderTime)
{
	bool bOk = false;
	if (!View.bDisableDistanceBasedFadeTransitions)
	{
		bOk = true;
		TemporalLODLag = CVarLODTemporalLag.GetValueOnRenderThread();
		if (TemporalLODTime[1] < LastRenderTime - TemporalLODLag)
		{
			if (TemporalLODTime[0] < TemporalLODTime[1])
			{
				TemporalLODViewOrigin[0] = TemporalLODViewOrigin[1];
				TemporalLODTime[0] = TemporalLODTime[1];
			}
			TemporalLODViewOrigin[1] = View.ViewMatrices.GetViewOrigin();
			TemporalLODTime[1] = LastRenderTime;
			if (TemporalLODTime[1] <= TemporalLODTime[0])
			{
				bOk = false; // we are paused or something or otherwise didn't get a good sample
			}
		}
	}
	if (!bOk)
	{
		TemporalLODViewOrigin[0] = View.ViewMatrices.GetViewOrigin();
		TemporalLODViewOrigin[1] = View.ViewMatrices.GetViewOrigin();
		TemporalLODTime[0] = LastRenderTime;
		TemporalLODTime[1] = LastRenderTime;
		TemporalLODLag = 0.0f;
	}
}

IMPLEMENT_STATIC_UNIFORM_BUFFER_SLOT(WorkingColorSpace);
IMPLEMENT_STATIC_UNIFORM_BUFFER_STRUCT(FWorkingColorSpaceShaderParameters, "WorkingColorSpace", WorkingColorSpace);

void FDefaultWorkingColorSpaceUniformBuffer::Update(FRHICommandListBase& RHICmdList, const UE::Color::FColorSpace& InColorSpace)
{
	using namespace UE::Color;

	const FVector2d& White = InColorSpace.GetWhiteChromaticity();
	const FVector2d ACES_D60 = GetWhitePoint(EWhitePoint::ACES_D60);

	FWorkingColorSpaceShaderParameters Parameters;
	Parameters.ToXYZ = Transpose<float>(InColorSpace.GetRgbToXYZ());
	Parameters.FromXYZ = Transpose<float>(InColorSpace.GetXYZToRgb());

	Parameters.ToAP1 = Transpose<float>(FColorSpaceTransform(InColorSpace, FColorSpace(EColorSpace::ACESAP1)));
	Parameters.FromAP1 = Parameters.ToAP1.Inverse();
	
	Parameters.ToAP0 = Transpose<float>(FColorSpaceTransform(InColorSpace, FColorSpace(EColorSpace::ACESAP0)));
	Parameters.FromAP0 = Parameters.ToAP0.Inverse();

	Parameters.bIsSRGB = InColorSpace.IsSRGB();

	SetContents(RHICmdList, Parameters);
}

TGlobalResource<FDefaultWorkingColorSpaceUniformBuffer> GDefaultWorkingColorSpaceUniformBuffer;


FSimpleElementCollector::FSimpleElementCollector() :
	FPrimitiveDrawInterface(nullptr)
{}

FSimpleElementCollector::~FSimpleElementCollector()
{
	// Cleanup the dynamic resources.
	for(int32 ResourceIndex = 0;ResourceIndex < DynamicResources.Num();ResourceIndex++)
	{
		//release the resources before deleting, they will delete themselves
		DynamicResources[ResourceIndex]->ReleasePrimitiveResource();
	}
}

void FSimpleElementCollector::SetHitProxy(HHitProxy* HitProxy)
{
	if (HitProxy)
	{
		HitProxyId = HitProxy->Id;
	}
	else
	{
		HitProxyId = FHitProxyId();
	}
}

void FSimpleElementCollector::DrawSprite(
	const FVector& Position,
	float SizeX,
	float SizeY,
	const FTexture* Sprite,
	const FLinearColor& Color,
	uint8 DepthPriorityGroup,
	float U,
	float UL,
	float V,
	float VL,
	uint8 BlendMode,
	float OpacityMaskRefVal
	)
{
	FBatchedElements& Elements = DepthPriorityGroup == SDPG_World ? BatchedElements : TopBatchedElements;

	Elements.AddSprite(
		Position,
		SizeX,
		SizeY,
		Sprite,
		Color,
		HitProxyId,
		U,
		UL,
		V,
		VL,
		BlendMode,
		OpacityMaskRefVal
		);
}

void FSimpleElementCollector::DrawLine(
	const FVector& Start,
	const FVector& End,
	const FLinearColor& Color,
	uint8 DepthPriorityGroup,
	float Thickness/* = 0.0f*/,
	float DepthBias/* = 0.0f*/,
	bool bScreenSpace/* = false*/
	)
{
	FBatchedElements& Elements = DepthPriorityGroup == SDPG_World ? BatchedElements : TopBatchedElements;

	Elements.AddLine(
		Start,
		End,
		Color,
		HitProxyId,
		Thickness,
		DepthBias,
		bScreenSpace
		);
}


void FSimpleElementCollector::DrawTranslucentLine(
	const FVector& Start,
	const FVector& End,
	const FLinearColor& Color,
	uint8 DepthPriorityGroup,
	float Thickness/* = 0.0f*/,
	float DepthBias/* = 0.0f*/,
	bool bScreenSpace/* = false*/
)
{
	FBatchedElements& Elements = DepthPriorityGroup == SDPG_World ? BatchedElements : TopBatchedElements;

	Elements.AddTranslucentLine(
		Start,
		End,
		Color,
		HitProxyId,
		Thickness,
		DepthBias,
		bScreenSpace
	);
}


void FSimpleElementCollector::DrawPoint(
	const FVector& Position,
	const FLinearColor& Color,
	float PointSize,
	uint8 DepthPriorityGroup
	)
{
	FBatchedElements& Elements = DepthPriorityGroup == SDPG_World ? BatchedElements : TopBatchedElements;

	Elements.AddPoint(
		Position,
		PointSize,
		Color,
		HitProxyId
		);
}

void FDynamicPrimitiveResource::InitPrimitiveResource()
{
	InitPrimitiveResource(FRHICommandListImmediate::Get());
}

void FSimpleElementCollector::RegisterDynamicResource(FDynamicPrimitiveResource* DynamicResource)
{
	// Add the dynamic resource to the list of resources to cleanup on destruction.
	DynamicResources.Add(DynamicResource);

	// Initialize the dynamic resource immediately.
	DynamicResource->InitPrimitiveResource(FRHICommandListImmediate::Get());
}

void FSimpleElementCollector::DrawBatchedElements(FRHICommandList& RHICmdList, const FMeshPassProcessorRenderState& DrawRenderState, const FSceneView& InView, EBlendModeFilter::Type Filter, ESceneDepthPriorityGroup DepthPriorityGroup, uint32 NumInstances) const
{
	const FBatchedElements& Elements = DepthPriorityGroup == SDPG_World ? BatchedElements : TopBatchedElements;

	// Draw the batched elements.
	Elements.Draw(
		RHICmdList,
		DrawRenderState,
		InView.GetFeatureLevel(),
		InView,
		InView.Family->EngineShowFlags.HitProxies,
		1.0f,
		Filter,
		NumInstances
		);
}

void FSimpleElementCollector::AddAllocationInfo(FAllocationInfo& AllocationInfo) const
{
	BatchedElements.AddAllocationInfo(AllocationInfo.BatchedElements);
	TopBatchedElements.AddAllocationInfo(AllocationInfo.TopBatchedElements);
	AllocationInfo.NumDynamicResources += DynamicResources.Num();
}

void FSimpleElementCollector::Reserve(const FAllocationInfo& AllocationInfo)
{
	BatchedElements.Reserve(AllocationInfo.BatchedElements);
	TopBatchedElements.Reserve(AllocationInfo.TopBatchedElements);
	DynamicResources.Reserve(AllocationInfo.NumDynamicResources);
}

void FSimpleElementCollector::Append(FSimpleElementCollector& Other)
{
	BatchedElements.Append(Other.BatchedElements);
	TopBatchedElements.Append(Other.TopBatchedElements);
	DynamicResources.Append(Other.DynamicResources);
	Other.DynamicResources.Empty();
}

FMeshBatchAndRelevance::FMeshBatchAndRelevance(const FMeshBatch& InMesh, const FPrimitiveSceneProxy* InPrimitiveSceneProxy, ERHIFeatureLevel::Type FeatureLevel) :
	Mesh(&InMesh),
	PrimitiveSceneProxy(InPrimitiveSceneProxy)
{
	if (InMesh.MaterialRenderProxy)
	{
		const FMaterial& Material = InMesh.MaterialRenderProxy->GetIncompleteMaterialWithFallback(FeatureLevel);
		bHasOpaqueMaterial = IsOpaqueBlendMode(Material);
		bHasMaskedMaterial = IsMaskedBlendMode(Material);
	}
	else
	{
		bHasOpaqueMaterial = false;
		bHasMaskedMaterial = false;
	}

	bRenderInMainPass = PrimitiveSceneProxy ? PrimitiveSceneProxy->ShouldRenderInMainPass() : false;
}

#if RHI_RAYTRACING

FRayTracingInstanceCollector::FRayTracingInstanceCollector(
	ERHIFeatureLevel::Type InFeatureLevel,
	FSceneRenderingBulkObjectAllocator& InBulkAllocator,
	bool bInTrackReferencedGeometry)
	: FMeshElementCollector(InFeatureLevel, InBulkAllocator, FMeshElementCollector::ECommitFlags::DeferAll)
	, bTrackReferencedGeometry(bInTrackReferencedGeometry)
{
}

void FRayTracingInstanceCollector::AddViewMeshArrays(
	const FSceneView* InView,
	FGPUScenePrimitiveCollector* DynamicPrimitiveCollector)
{
	Views.Add(InView);
	RDGPooledBuffers.Add({});

	FMeshElementCollector::AddViewMeshArrays(
		InView,
		nullptr,
		nullptr,
		DynamicPrimitiveCollector
#if UE_ENABLE_DEBUG_DRAWING
		, nullptr
#endif
	);
}

void FRayTracingInstanceCollector::SetVisibilityMap(uint32 InVisibilityMap)
{
	VisibilityMap = InVisibilityMap;
}

void FRayTracingInstanceCollector::AddRayTracingInstance(FRayTracingInstance Instance)
{
	AddRayTracingInstance(0, MoveTemp(Instance));
}

void FRayTracingInstanceCollector::AddRayTracingInstance(int32 ViewIndex, FRayTracingInstance Instance)
{
	FRayTracingInstanceAndViewIndex Tmp;
	Tmp.Instance = MoveTemp(Instance);
	Tmp.ViewIndex = ViewIndex;

	RayTracingInstances.AddElement(MoveTemp(Tmp));
}

void FRayTracingInstanceCollector::AddReferencedGeometry(RayTracing::FGeometryHandle Geometry)
{
	if (bTrackReferencedGeometry)
	{
		ReferencedGeometries.Add(Geometry);
	}
}

void FRayTracingInstanceCollector::AddReferencedGeometryGroup(RayTracing::FGeometryGroupHandle GeometryGroup)
{
	if (bTrackReferencedGeometry)
	{
		ReferencedGeometryGroups.Add(GeometryGroup);
	}
}

void FRayTracingInstanceCollector::AddReferencedGeometryGroupForDynamicUpdate(RayTracing::FGeometryGroupHandle GeometryGroup)
{
	if (bTrackReferencedGeometry)
	{
		ReferencedGeometryGroupsForDynamicUpdate.Add(GeometryGroup);
	}
}

void FRayTracingInstanceCollector::AddRayTracingGeometryUpdate(FRayTracingDynamicGeometryUpdateParams Params)
{
	AddRayTracingGeometryUpdate(0, MoveTemp(Params));
}

void FRayTracingInstanceCollector::AddRayTracingGeometryUpdate(int32 ViewIndex, FRayTracingDynamicGeometryUpdateParams Params)
{
	FRayTracingDynamicGeometryUpdateRequest UpdateRequest;
	UpdateRequest.Params = MoveTemp(Params);
	UpdateRequest.ViewIndex = ViewIndex;

	RayTracingGeometriesToUpdate.AddElement(MoveTemp(UpdateRequest));
}

#endif

FMeshElementCollector::FMeshElementCollector(ERHIFeatureLevel::Type InFeatureLevel, FSceneRenderingBulkObjectAllocator& InBulkAllocator, ECommitFlags InCommitFlags) :
	OneFrameResources(InBulkAllocator),
	PrimitiveSceneProxy(NULL),
	DynamicReadBuffer(nullptr),
	FeatureLevel(InFeatureLevel),
	CommitFlags(InCommitFlags),
	bUseGPUScene(UseGPUScene(GShaderPlatformForFeatureLevel[FeatureLevel]))
{
}

FMeshElementCollector::~FMeshElementCollector()
{
	for (FMaterialRenderProxy* Proxy : MaterialProxiesToDelete)
	{
		delete Proxy;
	}
	MaterialProxiesToDelete.Empty();
}

void FMeshElementCollector::RegisterOneFrameMaterialProxy(FMaterialRenderProxy* Proxy)
{
	check(Proxy);
	Proxy->MarkTransient();
	MaterialProxiesToDelete.Add(Proxy);
}

FPrimitiveDrawInterface* FMeshElementCollector::GetPDI(int32 ViewIndex)
{
	return SimpleElementCollectors[ViewIndex];
}

#if UE_ENABLE_DEBUG_DRAWING
FPrimitiveDrawInterface* FMeshElementCollector::GetDebugPDI(int32 ViewIndex)
{
	return DebugSimpleElementCollectors[ViewIndex];
}
#endif

void FMeshElementCollector::SetPrimitive(const FPrimitiveSceneProxy* InPrimitiveSceneProxy, const FHitProxyId& DefaultHitProxyId)
{
	check(InPrimitiveSceneProxy);
	PrimitiveSceneProxy = InPrimitiveSceneProxy;

	for (int32 ViewIndex = 0; ViewIndex < SimpleElementCollectors.Num(); ViewIndex++)
	{
		if (SimpleElementCollectors[ViewIndex])
		{
			SimpleElementCollectors[ViewIndex]->HitProxyId = DefaultHitProxyId;
		}
	}

	for (int32 ViewIndex = 0; ViewIndex < MeshIdInPrimitivePerView.Num(); ++ViewIndex)
	{
		MeshIdInPrimitivePerView[ViewIndex] = 0;
	}

#if UE_ENABLE_DEBUG_DRAWING
	for (int32 ViewIndex = 0; ViewIndex < DebugSimpleElementCollectors.Num(); ViewIndex++)
	{
		DebugSimpleElementCollectors[ViewIndex]->HitProxyId = DefaultHitProxyId;
	}
#endif
}

void FMeshElementCollector::Start(
	FRHICommandList& InRHICmdList,
	FGlobalDynamicVertexBuffer& InDynamicVertexBuffer,
	FGlobalDynamicIndexBuffer& InDynamicIndexBuffer,
	FGlobalDynamicReadBuffer& InDynamicReadBuffer)
{
	check(!RHICmdList);
	RHICmdList = &InRHICmdList;
	DynamicVertexBuffer = &InDynamicVertexBuffer;
	DynamicIndexBuffer = &InDynamicIndexBuffer;
	DynamicReadBuffer = &InDynamicReadBuffer;
}

void FMeshElementCollector::AddViewMeshArrays(
	const FSceneView* InView,
	TArray<FMeshBatchAndRelevance, SceneRenderingAllocator>* ViewMeshes,
	FSimpleElementCollector* ViewSimpleElementCollector,
	FGPUScenePrimitiveCollector* DynamicPrimitiveCollector
#if UE_ENABLE_DEBUG_DRAWING
	, FSimpleElementCollector* DebugSimpleElementCollector
#endif
)
{
	check(RHICmdList);

	Views.Add(InView);
	MeshIdInPrimitivePerView.Add(0);
	MeshBatches.Add(ViewMeshes);
	NumMeshBatchElementsPerView.Add(0);
	SimpleElementCollectors.Add(ViewSimpleElementCollector);
	DynamicPrimitiveCollectorPerView.Add(DynamicPrimitiveCollector);

#if UE_ENABLE_DEBUG_DRAWING
	//Assign the debug draw only simple element collector per view	
	if (DebugSimpleElementCollector)
	{
		DebugSimpleElementCollectors.Add(DebugSimpleElementCollector);
	}
#endif
}

void FMeshElementCollector::ClearViewMeshArrays()
{
	Views.Reset();
	MeshIdInPrimitivePerView.Reset();
	MeshBatches.Reset();
	NumMeshBatchElementsPerView.Reset();
	SimpleElementCollectors.Reset();
	DynamicPrimitiveCollectorPerView.Reset();
#if UE_ENABLE_DEBUG_DRAWING
	DebugSimpleElementCollectors.Reset();
#endif
}

void FMeshElementCollector::Commit()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FMeshElementCollector::Commit);
	check(RHICmdList);

	for (TPair<FGPUScenePrimitiveCollector*, FMeshBatch*> Pair : MeshBatchesForGPUScene)
	{
		GetRendererModule().AddMeshBatchToGPUScene(Pair.Key, *Pair.Value);
	}

	for (TPair<FMaterialRenderProxy*, bool> Parameters : MaterialProxiesToInvalidate)
	{
		Parameters.Key->InvalidateUniformExpressionCache(Parameters.Value);
	}

	for (const FMaterialRenderProxy* Proxy : MaterialProxiesToUpdate)
	{
		Proxy->UpdateUniformExpressionCacheIfNeeded(*RHICmdList, FeatureLevel);
	}

	MeshBatchesForGPUScene.Empty();
	MaterialProxiesToInvalidate.Empty();
	MaterialProxiesToUpdate.Empty();
}

void FMeshElementCollector::Finish()
{
	SCOPED_NAMED_EVENT(FMeshElementCollector_Finish, FColor::Magenta);

	Commit();
	ClearViewMeshArrays();
	DynamicIndexBuffer = nullptr;
	DynamicVertexBuffer = nullptr;
	DynamicReadBuffer = nullptr;
	RHICmdList = nullptr;
}

void FMeshElementCollector::CacheUniformExpressions(FMaterialRenderProxy* Proxy, bool bRecreateUniformBuffer)
{
	check(Proxy);
	if (EnumHasAnyFlags(CommitFlags, ECommitFlags::DeferMaterials))
	{
		MaterialProxiesToInvalidate.Emplace(Proxy, bRecreateUniformBuffer);
	}
	else
	{
		Proxy->InvalidateUniformExpressionCache(bRecreateUniformBuffer);
	}
}

void FMeshElementCollector::AddMesh(int32 ViewIndex, FMeshBatch& MeshBatch)
{
	DEFINE_LOG_CATEGORY_STATIC(FMeshElementCollector_AddMesh, Warning, All);

	if (MeshBatch.bCanApplyViewModeOverrides)
	{
		const FSceneView* View = Views[ViewIndex];

		ApplyViewModeOverrides(
			ViewIndex,
			View->Family->EngineShowFlags,
			View->GetFeatureLevel(),
			PrimitiveSceneProxy,
			MeshBatch.bUseWireframeSelectionColoring,
			MeshBatch,
			*this);
	}

	if (!MeshBatch.Validate(PrimitiveSceneProxy, FeatureLevel))
	{
		return;
	}

	MeshBatch.PreparePrimitiveUniformBuffer(PrimitiveSceneProxy, FeatureLevel);

	if (bUseGPUScene && MeshBatch.VertexFactory->GetPrimitiveIdStreamIndex(FeatureLevel, EVertexInputStreamType::Default) >= 0)
	{
		if (EnumHasAnyFlags(CommitFlags, ECommitFlags::DeferGPUScene))
		{
			MeshBatchesForGPUScene.Emplace(DynamicPrimitiveCollectorPerView[ViewIndex], &MeshBatch);
		}
		else
		{
			GetRendererModule().AddMeshBatchToGPUScene(DynamicPrimitiveCollectorPerView[ViewIndex], MeshBatch);
		}
	}

	if (EnumHasAnyFlags(CommitFlags, ECommitFlags::DeferMaterials))
	{
		MaterialProxiesToUpdate.Emplace(MeshBatch.MaterialRenderProxy);
	}
	else
	{
		MeshBatch.MaterialRenderProxy->UpdateUniformExpressionCacheIfNeeded(*RHICmdList, FeatureLevel);
	}

	MeshBatch.MeshIdInPrimitive = MeshIdInPrimitivePerView[ViewIndex];
	++MeshIdInPrimitivePerView[ViewIndex];

	NumMeshBatchElementsPerView[ViewIndex] += MeshBatch.Elements.Num();

	if (MeshBatches[ViewIndex])
	{
		MeshBatches[ViewIndex]->Emplace(MeshBatch, PrimitiveSceneProxy, FeatureLevel);
	}
}

FDynamicPrimitiveUniformBuffer::FDynamicPrimitiveUniformBuffer() = default;
FDynamicPrimitiveUniformBuffer::~FDynamicPrimitiveUniformBuffer()
{
	UniformBuffer.ReleaseResource();
}


void FDynamicPrimitiveUniformBuffer::Set(FRHICommandListBase& RHICmdList, FPrimitiveUniformShaderParametersBuilder& Builder)
{
	UniformBuffer.BufferUsage = UniformBuffer_SingleFrame;
	UniformBuffer.SetContents(RHICmdList, Builder.Build());
	UniformBuffer.InitResource(RHICmdList);
}

void FDynamicPrimitiveUniformBuffer::Set(
	FRHICommandListBase& RHICmdList,
	const FMatrix& LocalToWorld,
	const FMatrix& PreviousLocalToWorld,
	const FVector& ActorPositionWS,
	const FBoxSphereBounds& WorldBounds,
	const FBoxSphereBounds& LocalBounds,
	const FBoxSphereBounds& PreSkinnedLocalBounds,
	bool bReceivesDecals,
	bool bHasPrecomputedVolumetricLightmap,
	bool bOutputVelocity,
	const FCustomPrimitiveData* CustomPrimitiveData)
{
	Set(
		RHICmdList,
		FPrimitiveUniformShaderParametersBuilder{}
		.Defaults()
			.LocalToWorld(LocalToWorld)
			.PreviousLocalToWorld(PreviousLocalToWorld)
			.ActorWorldPosition(ActorPositionWS)
			.WorldBounds(WorldBounds)
			.LocalBounds(LocalBounds)
			.PreSkinnedLocalBounds(PreSkinnedLocalBounds)
			.ReceivesDecals(bReceivesDecals)
			.OutputVelocity(bOutputVelocity)
			.UseVolumetricLightmap(bHasPrecomputedVolumetricLightmap)
			.CustomPrimitiveData(CustomPrimitiveData)
	);
}

void FDynamicPrimitiveUniformBuffer::Set(
	FRHICommandListBase& RHICmdList,
	const FMatrix& LocalToWorld,
	const FMatrix& PreviousLocalToWorld,
	const FBoxSphereBounds& WorldBounds,
	const FBoxSphereBounds& LocalBounds,
	const FBoxSphereBounds& PreSkinnedLocalBounds,
	bool bReceivesDecals,
	bool bHasPrecomputedVolumetricLightmap,
	bool bOutputVelocity,
	const FCustomPrimitiveData* CustomPrimitiveData)
{
	Set(RHICmdList, LocalToWorld, PreviousLocalToWorld, WorldBounds.Origin, WorldBounds, LocalBounds, PreSkinnedLocalBounds, bReceivesDecals, bHasPrecomputedVolumetricLightmap, bOutputVelocity, CustomPrimitiveData);
}

void FDynamicPrimitiveUniformBuffer::Set(
	FRHICommandListBase& RHICmdList,
	const FMatrix& LocalToWorld,
	const FMatrix& PreviousLocalToWorld,
	const FBoxSphereBounds& WorldBounds,
	const FBoxSphereBounds& LocalBounds,
	const FBoxSphereBounds& PreSkinnedLocalBounds,
	bool bReceivesDecals,
	bool bHasPrecomputedVolumetricLightmap,
	bool bOutputVelocity)
{
	Set(RHICmdList, LocalToWorld, PreviousLocalToWorld, WorldBounds, LocalBounds, PreSkinnedLocalBounds, bReceivesDecals, bHasPrecomputedVolumetricLightmap, bOutputVelocity, nullptr);
}

void FDynamicPrimitiveUniformBuffer::Set(
	FRHICommandListBase& RHICmdList,
	const FMatrix& LocalToWorld,
	const FMatrix& PreviousLocalToWorld,
	const FBoxSphereBounds& WorldBounds,
	const FBoxSphereBounds& LocalBounds,
	bool bReceivesDecals,
	bool bHasPrecomputedVolumetricLightmap,
	bool bOutputVelocity)
{
	Set(RHICmdList, LocalToWorld, PreviousLocalToWorld, WorldBounds, LocalBounds, LocalBounds, bReceivesDecals, bHasPrecomputedVolumetricLightmap, bOutputVelocity, nullptr);
}

void FDynamicPrimitiveUniformBuffer::Set(
	const FMatrix& LocalToWorld,
	const FMatrix& PreviousLocalToWorld,
	const FVector& ActorPositionWS,
	const FBoxSphereBounds& WorldBounds,
	const FBoxSphereBounds& LocalBounds,
	const FBoxSphereBounds& PreSkinnedLocalBounds,
	bool bReceivesDecals,
	bool bHasPrecomputedVolumetricLightmap,
	bool bOutputVelocity,
	const FCustomPrimitiveData* CustomPrimitiveData)
{
	Set(FRHICommandListImmediate::Get(), LocalToWorld, PreviousLocalToWorld, ActorPositionWS, WorldBounds, LocalBounds, PreSkinnedLocalBounds, bReceivesDecals, bHasPrecomputedVolumetricLightmap, bOutputVelocity, CustomPrimitiveData);
}

void FDynamicPrimitiveUniformBuffer::Set(
	const FMatrix& LocalToWorld,
	const FMatrix& PreviousLocalToWorld,
	const FBoxSphereBounds& WorldBounds,
	const FBoxSphereBounds& LocalBounds,
	const FBoxSphereBounds& PreSkinnedLocalBounds,
	bool bReceivesDecals,
	bool bHasPrecomputedVolumetricLightmap,
	bool bOutputVelocity,
	const FCustomPrimitiveData* CustomPrimitiveData)
{
	Set(FRHICommandListImmediate::Get(), LocalToWorld, PreviousLocalToWorld, WorldBounds.Origin, WorldBounds, LocalBounds, PreSkinnedLocalBounds, bReceivesDecals, bHasPrecomputedVolumetricLightmap, bOutputVelocity, CustomPrimitiveData);
}

void FDynamicPrimitiveUniformBuffer::Set(
	const FMatrix& LocalToWorld,
	const FMatrix& PreviousLocalToWorld,
	const FBoxSphereBounds& WorldBounds,
	const FBoxSphereBounds& LocalBounds,
	const FBoxSphereBounds& PreSkinnedLocalBounds,
	bool bReceivesDecals,
	bool bHasPrecomputedVolumetricLightmap,
	bool bOutputVelocity)
{
	Set(FRHICommandListImmediate::Get(), LocalToWorld, PreviousLocalToWorld, WorldBounds, LocalBounds, PreSkinnedLocalBounds, bReceivesDecals, bHasPrecomputedVolumetricLightmap, bOutputVelocity, nullptr);
}

void FDynamicPrimitiveUniformBuffer::Set(
	const FMatrix& LocalToWorld,
	const FMatrix& PreviousLocalToWorld,
	const FBoxSphereBounds& WorldBounds,
	const FBoxSphereBounds& LocalBounds,
	bool bReceivesDecals,
	bool bHasPrecomputedVolumetricLightmap,
	bool bOutputVelocity)
{
	Set(FRHICommandListImmediate::Get(), LocalToWorld, PreviousLocalToWorld, WorldBounds, LocalBounds, LocalBounds, bReceivesDecals, bHasPrecomputedVolumetricLightmap, bOutputVelocity, nullptr);
}

FLightMapInteraction FLightMapInteraction::Texture(
	const class ULightMapTexture2D* const* InTextures,
	const ULightMapTexture2D* InSkyOcclusionTexture,
	const ULightMapTexture2D* InAOMaterialMaskTexture,
	const FVector4f* InCoefficientScales,
	const FVector4f* InCoefficientAdds,
	const FVector2D& InCoordinateScale,
	const FVector2D& InCoordinateBias,
	bool bUseHighQualityLightMaps)
{
	FLightMapInteraction Result;
	Result.Type = LMIT_Texture;

#if ALLOW_LQ_LIGHTMAPS && ALLOW_HQ_LIGHTMAPS
	// however, if simple and directional are allowed, then we must use the value passed in,
	// and then cache the number as well
	Result.bAllowHighQualityLightMaps = bUseHighQualityLightMaps;
	if (bUseHighQualityLightMaps)
	{
		Result.NumLightmapCoefficients = NUM_HQ_LIGHTMAP_COEF;
	}
	else
	{
		Result.NumLightmapCoefficients = NUM_LQ_LIGHTMAP_COEF;
	}
#endif

	//copy over the appropriate textures and scales
	if (bUseHighQualityLightMaps)
	{
#if ALLOW_HQ_LIGHTMAPS
		Result.HighQualityTexture = InTextures[0];
		Result.SkyOcclusionTexture = InSkyOcclusionTexture;
		Result.AOMaterialMaskTexture = InAOMaterialMaskTexture;
		for(uint32 CoefficientIndex = 0;CoefficientIndex < NUM_HQ_LIGHTMAP_COEF;CoefficientIndex++)
		{
			Result.HighQualityCoefficientScales[CoefficientIndex] = InCoefficientScales[CoefficientIndex];
			Result.HighQualityCoefficientAdds[CoefficientIndex] = InCoefficientAdds[CoefficientIndex];
		}
#endif
	}

	// NOTE: In PC editor we cache both Simple and Directional textures as we may need to dynamically switch between them
	if( GIsEditor || !bUseHighQualityLightMaps )
	{
#if ALLOW_LQ_LIGHTMAPS
		Result.LowQualityTexture = InTextures[1];
		for(uint32 CoefficientIndex = 0;CoefficientIndex < NUM_LQ_LIGHTMAP_COEF;CoefficientIndex++)
		{
			Result.LowQualityCoefficientScales[CoefficientIndex] = InCoefficientScales[ LQ_LIGHTMAP_COEF_INDEX + CoefficientIndex ];
			Result.LowQualityCoefficientAdds[CoefficientIndex] = InCoefficientAdds[ LQ_LIGHTMAP_COEF_INDEX + CoefficientIndex ];
		}
#endif
	}

	Result.CoordinateScale = InCoordinateScale;
	Result.CoordinateBias = InCoordinateBias;
	return Result;
}

FLightMapInteraction FLightMapInteraction::InitVirtualTexture(
	const ULightMapVirtualTexture2D* VirtualTexture,
	const FVector4f* InCoefficientScales,
	const FVector4f* InCoefficientAdds,
	const FVector2D& InCoordinateScale,
	const FVector2D& InCoordinateBias,
	bool bAllowHighQualityLightMaps)
{
	FLightMapInteraction Result;
	Result.Type = LMIT_Texture;

#if ALLOW_LQ_LIGHTMAPS && ALLOW_HQ_LIGHTMAPS
	// however, if simple and directional are allowed, then we must use the value passed in,
	// and then cache the number as well
	Result.bAllowHighQualityLightMaps = bAllowHighQualityLightMaps;
	if (bAllowHighQualityLightMaps)
	{
		Result.NumLightmapCoefficients = NUM_HQ_LIGHTMAP_COEF;
	}
	else
	{
		Result.NumLightmapCoefficients = NUM_LQ_LIGHTMAP_COEF;
	}
#endif

	//copy over the appropriate textures and scales
	if (bAllowHighQualityLightMaps)
	{
#if ALLOW_HQ_LIGHTMAPS
		Result.VirtualTexture = VirtualTexture;
		for (uint32 CoefficientIndex = 0; CoefficientIndex < NUM_HQ_LIGHTMAP_COEF; CoefficientIndex++)
		{
			Result.HighQualityCoefficientScales[CoefficientIndex] = InCoefficientScales[CoefficientIndex];
			Result.HighQualityCoefficientAdds[CoefficientIndex] = InCoefficientAdds[CoefficientIndex];
		}
#endif
	}

	// NOTE: In PC editor we cache both Simple and Directional textures as we may need to dynamically switch between them
	if (GIsEditor || !bAllowHighQualityLightMaps)
	{
#if ALLOW_LQ_LIGHTMAPS
		Result.VirtualTexture = VirtualTexture;
		for (uint32 CoefficientIndex = 0; CoefficientIndex < NUM_LQ_LIGHTMAP_COEF; CoefficientIndex++)
		{
			Result.LowQualityCoefficientScales[CoefficientIndex] = InCoefficientScales[LQ_LIGHTMAP_COEF_INDEX + CoefficientIndex];
			Result.LowQualityCoefficientAdds[CoefficientIndex] = InCoefficientAdds[LQ_LIGHTMAP_COEF_INDEX + CoefficientIndex];
		}
#endif
	}

	Result.CoordinateScale = InCoordinateScale;
	Result.CoordinateBias = InCoordinateBias;
	return Result;
}

float ComputeBoundsScreenRadiusSquared(const FVector4& BoundsOrigin, const float SphereRadius, const FVector4& ViewOrigin, const FMatrix& ProjMatrix)
{
	// ignore perspective foreshortening for orthographic projections
	const float DistSqr = FVector::DistSquared(BoundsOrigin, ViewOrigin) * ProjMatrix.M[2][3];

	// Get projection multiple accounting for view scaling.
	const float ScreenMultiple = FMath::Max(0.5f * ProjMatrix.M[0][0], 0.5f * ProjMatrix.M[1][1]);

	// Calculate screen-space projected radius
	return FMath::Square(ScreenMultiple * SphereRadius) / FMath::Max(1.0f, DistSqr);
}

/** Runtime comparison version of ComputeTemporalLODBoundsScreenSize that avoids a square root */
static float ComputeTemporalLODBoundsScreenRadiusSquared(const FVector& Origin, const float SphereRadius, const FSceneView& View, int32 SampleIndex)
{
	return ComputeBoundsScreenRadiusSquared(Origin, SphereRadius, View.GetTemporalLODOrigin(SampleIndex), View.ViewMatrices.GetViewToClip());
}

float ComputeBoundsScreenRadiusSquared(const FVector4& Origin, const float SphereRadius, const FSceneView& View)
{
	return ComputeBoundsScreenRadiusSquared(Origin, SphereRadius, View.ViewMatrices.GetViewOrigin(), View.ViewMatrices.GetViewToClip());
}

float ComputeBoundsScreenSize( const FVector4& Origin, const float SphereRadius, const FSceneView& View )
{
	return ComputeBoundsScreenSize(Origin, SphereRadius, View.ViewMatrices.GetViewOrigin(), View.ViewMatrices.GetViewToClip());
}

float ComputeTemporalLODBoundsScreenSize( const FVector& Origin, const float SphereRadius, const FSceneView& View, int32 SampleIndex )
{
	return ComputeBoundsScreenSize(Origin, SphereRadius, View.GetTemporalLODOrigin(SampleIndex), View.ViewMatrices.GetViewToClip());
}

float ComputeBoundsScreenSize(const FVector4& BoundsOrigin, const float SphereRadius, const FVector4& ViewOrigin, const FMatrix& ProjMatrix)
{
	const float Dist = FVector::Dist(BoundsOrigin, ViewOrigin);

	// Get projection multiple accounting for view scaling.
	const float ScreenMultiple = FMath::Max(0.5f * ProjMatrix.M[0][0], 0.5f * ProjMatrix.M[1][1]);

	// Calculate screen-space projected radius
	const float ScreenRadius = ScreenMultiple * SphereRadius / FMath::Max(1.0f, Dist);

	// For clarity, we end up comparing the diameter
	return ScreenRadius * 2.0f;
}

float ComputeBoundsDrawDistance(const float ScreenSize, const float SphereRadius, const FMatrix& ProjMatrix)
{
	// Get projection multiple accounting for view scaling.
	const float ScreenMultiple = FMath::Max(0.5f * ProjMatrix.M[0][0], 0.5f * ProjMatrix.M[1][1]);

	// ScreenSize is the projected diameter, so halve it
	const float ScreenRadius = FMath::Max(UE_SMALL_NUMBER, ScreenSize * 0.5f);

	// Invert the calcs in ComputeBoundsScreenSize
	return (ScreenMultiple * SphereRadius) / ScreenRadius;
}

int8 ComputeTemporalStaticMeshLOD( const FStaticMeshRenderData* RenderData, const FVector4& Origin, const float SphereRadius, const FSceneView& View, int32 MinLOD, float FactorScale, int32 SampleIndex )
{
	const int32 NumLODs = MAX_STATIC_MESH_LODS;

	const float ScreenRadiusSquared = ComputeTemporalLODBoundsScreenRadiusSquared(Origin, SphereRadius, View, SampleIndex);
	const float ScreenSizeScale = FactorScale * View.LODDistanceFactor;

	// Walk backwards and return the first matching LOD
	for(int32 LODIndex = NumLODs - 1 ; LODIndex >= 0 ; --LODIndex)
	{
		const float MeshScreenSize = RenderData->ScreenSize[LODIndex].GetValue() * ScreenSizeScale;
		
		if(FMath::Square(MeshScreenSize * 0.5f) > ScreenRadiusSquared)
		{
			return FMath::Max(LODIndex, MinLOD);
		}
	}

	return MinLOD;
}

// Ensure we always use the left eye when selecting lods to avoid divergent selections in stereo
const FSceneView& GetLODView(const FSceneView& InView)
{
	if (UNLIKELY(IStereoRendering::IsStereoEyeView(InView) && GEngine->StereoRenderingDevice.IsValid()))
	{
		uint32 LODViewIndex = GEngine->StereoRenderingDevice->GetLODViewIndex();
		if (InView.Family && InView.Family->Views.IsValidIndex(LODViewIndex))
		{
			return *InView.Family->Views[LODViewIndex];
		}
	}

	return InView;
}

int8 ComputeStaticMeshLOD( const FStaticMeshRenderData* RenderData, const FVector4& Origin, const float SphereRadius, const FSceneView& View, int32 MinLOD, float FactorScale )
{
	if (RenderData)
	{
		const int32 NumLODs = MAX_STATIC_MESH_LODS;
		const FSceneView& LODView = GetLODView(View);
		const float ScreenRadiusSquared = ComputeBoundsScreenRadiusSquared(Origin, SphereRadius, LODView);
		const float ScreenSizeScale = FactorScale * LODView.LODDistanceFactor;

		// Walk backwards and return the first matching LOD
		for (int32 LODIndex = NumLODs - 1; LODIndex >= 0; --LODIndex)
		{
			float MeshScreenSize = RenderData->ScreenSize[LODIndex].GetValue() * ScreenSizeScale;

			if (FMath::Square(MeshScreenSize * 0.5f) > ScreenRadiusSquared)
			{
				return FMath::Max(LODIndex, MinLOD);
			}
		}
	}

	return MinLOD;
}

FLODMask ComputeLODForMeshes(const TArray<class FStaticMeshBatchRelevance>& StaticMeshRelevances, const FSceneView& View, const FVector4& Origin, float SphereRadius, int32 ForcedLODLevel, float& OutScreenRadiusSquared, int8 CurFirstLODIdx, float ScreenSizeScale, bool bDitheredLODTransition)
{
	FLODMask LODToRender;
	const FSceneView& LODView = GetLODView(View);

	const int32 NumMeshes = StaticMeshRelevances.Num();

	// Handle forced LOD level first
	if (ForcedLODLevel >= 0)
	{
		OutScreenRadiusSquared = 0.0f;

		int32 MinLOD = 127, MaxLOD = 0;
		for (int32 MeshIndex = 0; MeshIndex < StaticMeshRelevances.Num(); ++MeshIndex)
		{
			const FStaticMeshBatchRelevance& Mesh = StaticMeshRelevances[MeshIndex];
			if (Mesh.ScreenSize > 0.0f && !Mesh.bOverlayMaterial)
			{
				MinLOD = FMath::Min(MinLOD, (int32)Mesh.GetLODIndex());
				MaxLOD = FMath::Max(MaxLOD, (int32)Mesh.GetLODIndex());
			}
		}
		MinLOD = FMath::Max(MinLOD, (int32)CurFirstLODIdx);

		// force LOD dither transition with the next LOD
		if (GDebugLODDitherSpeed > 0.0f && NumMeshes > 0 && bDitheredLODTransition && StaticMeshRelevances[0].bDitheredLODTransition && MaxLOD != 0)
		{
			int32 LOD0 = FMath::Clamp(ForcedLODLevel, MinLOD, MaxLOD);
			int32 LOD1 = FMath::Clamp(LOD0 + 1, MinLOD, MaxLOD);
			LODToRender.SetLODSample(LOD0, 0);
			LODToRender.SetLODSample(LOD1, 1);
		}
		else
		{
			LODToRender.SetLOD(FMath::Clamp(ForcedLODLevel, MinLOD, MaxLOD));
		}
	}
	else if (LODView.Family->EngineShowFlags.LOD && NumMeshes)
	{
		if (bDitheredLODTransition && StaticMeshRelevances[0].bDitheredLODTransition)
		{
			for (int32 SampleIndex = 0; SampleIndex < 2; SampleIndex++)
			{
				int32 MinLODFound = INT_MAX;
				int32 MaxLOD = 0;

				bool bFoundLOD = false;
				OutScreenRadiusSquared = ComputeTemporalLODBoundsScreenRadiusSquared(Origin, SphereRadius, LODView, SampleIndex);

				for (int32 MeshIndex = NumMeshes - 1; MeshIndex >= 0; --MeshIndex)
				{
					const FStaticMeshBatchRelevance& Mesh = StaticMeshRelevances[MeshIndex];
					// We skip overlay material meshes as they always use base mesh LOD
					if (Mesh.ScreenSize > 0.0f && !Mesh.bOverlayMaterial)
					{
						float MeshScreenSize = Mesh.ScreenSize * ScreenSizeScale;
						
						if (FMath::Square(MeshScreenSize * 0.5f) >= OutScreenRadiusSquared)
						{
							LODToRender.SetLODSample(Mesh.GetLODIndex(), SampleIndex);
							bFoundLOD = true;
							break;
						}

						MinLODFound = FMath::Min<int32>(MinLODFound, Mesh.GetLODIndex());
						MaxLOD = FMath::Max<int32>(MaxLOD, Mesh.GetLODIndex());
					}
				}
				// If no LOD was found matching the screen size, use the lowest in the array instead of LOD 0, to handle non-zero MinLOD
				if (!bFoundLOD)
				{
					LODToRender.SetLODSample(MinLODFound, SampleIndex);
				}

				// force LOD dither transition with the next LOD
				if (GDebugLODDitherSpeed > 0.0f && LODToRender.IsValid() && LODToRender.LODIndex0 == LODToRender.LODIndex1)
				{
					int32 LOD0 = LODToRender.LODIndex0;
					int32 LOD1 = FMath::Min(LOD0 + 1, MaxLOD);
					LODToRender.SetLODSample(LOD0, 0);
					LODToRender.SetLODSample(LOD1, 1);				
				}
			}
		}
		else
		{
			int32 MinLODFound = INT_MAX;
			bool bFoundLOD = false;
			OutScreenRadiusSquared = ComputeBoundsScreenRadiusSquared(Origin, SphereRadius, LODView);

			for (int32 MeshIndex = NumMeshes - 1; MeshIndex >= 0; --MeshIndex)
			{
				const FStaticMeshBatchRelevance& Mesh = StaticMeshRelevances[MeshIndex];

				float MeshScreenSize = Mesh.ScreenSize * ScreenSizeScale;
				// We skip overlay material meshes as they always use base mesh LOD
				if (FMath::Square(MeshScreenSize * 0.5f) >= OutScreenRadiusSquared && !Mesh.bOverlayMaterial)
				{
					LODToRender.SetLOD(Mesh.GetLODIndex());
					bFoundLOD = true;
					break;
				}

				MinLODFound = FMath::Min<int32>(MinLODFound, Mesh.GetLODIndex());
			}
			// If no LOD was found matching the screen size, use the lowest in the array instead of LOD 0, to handle non-zero MinLOD
			if (!bFoundLOD)
			{
				LODToRender.SetLOD(MinLODFound);
			}
		}
		LODToRender.ClampToFirstLOD(CurFirstLODIdx);
	}
	return LODToRender;
}

FLODMask ComputeLODForMeshes(const TArray<class FStaticMeshBatchRelevance>& StaticMeshRelevances, const FSceneView& View, const FVector4& BoundsOrigin, float BoundsSphereRadius, float InstanceSphereRadius, int32 ForcedLODLevel, float& OutScreenRadiusSquared, int8 CurFirstLODIdx, float ScreenSizeScale)
{
	if (ForcedLODLevel >= 0 || InstanceSphereRadius <= 0.f)
	{
		return ComputeLODForMeshes(StaticMeshRelevances, View, BoundsOrigin, BoundsSphereRadius, ForcedLODLevel, OutScreenRadiusSquared, CurFirstLODIdx, ScreenSizeScale);
	}

	// The bounds origin and radius are for a group of instances.
	// Compute the range of possible LODs within that bounds.
	// todo: InstanceSphereRadius isn't enough. Need to take into account maximum and minimum instance scale.
	const FSceneView& LODView = GetLODView(View);
	const FVector CameraPosition = LODView.ViewMatrices.GetViewOrigin();
	const FVector BoundsOriginToCamera = CameraPosition - BoundsOrigin;
	const float Distance = BoundsOriginToCamera.Length();
	const FVector BoundsOriginToCameraNorm = BoundsOriginToCamera / Distance;
	const float AdjustedBoundsSphereRadius = FMath::Max(BoundsSphereRadius - InstanceSphereRadius, 0.f);
	const FVector FarInstanceOrigin = BoundsOrigin - AdjustedBoundsSphereRadius * BoundsOriginToCameraNorm;
	const FVector NearInstanceOrigin = (Distance <= AdjustedBoundsSphereRadius) ? CameraPosition : (FVector)BoundsOrigin + AdjustedBoundsSphereRadius * BoundsOriginToCameraNorm;

	FLODMask MaxLod = ComputeLODForMeshes(StaticMeshRelevances, View, FarInstanceOrigin, InstanceSphereRadius, -1, OutScreenRadiusSquared, CurFirstLODIdx, ScreenSizeScale, false);
	FLODMask MinLod = ComputeLODForMeshes(StaticMeshRelevances, View, NearInstanceOrigin, InstanceSphereRadius, -1, OutScreenRadiusSquared, CurFirstLODIdx, ScreenSizeScale, false);

	FLODMask Result;
	Result.SetLODRange(MinLod.LODIndex0, MaxLod.LODIndex0);
	return Result;
}

FMobileDirectionalLightShaderParameters::FMobileDirectionalLightShaderParameters()
{
	FMemory::Memzero(*this);

	// light, default to black
	DirectionalLightColor = FLinearColor::Black;
	DirectionalLightDirectionAndShadowTransition = FVector4f(EForceInit::ForceInitToZero);
	DirectionalLightShadowMapChannelMask = 0xFF;

	// white texture should act like a shadowmap cleared to the farplane.
	DirectionalLightShadowTexture = GWhiteTexture->TextureRHI;
	DirectionalLightShadowSampler = TStaticSamplerState<SF_Point, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
	DirectionalLightShadowSize = FVector4f(EForceInit::ForceInitToZero);
	DirectionalLightDistanceFadeMADAndSpecularScale = FVector4f(EForceInit::ForceInitToZero);
	DirectionalLightNumCascades = 0;
	for (int32 i = 0; i < MAX_MOBILE_SHADOWCASCADES; ++i)
	{
		DirectionalLightScreenToShadow[i].SetIdentity();
		DirectionalLightShadowDistances[i] = FLT_MAX; // Unused cascades should compare > all scene depths
	}
	ShadowedBits = 0;
}

FViewUniformShaderParameters::FViewUniformShaderParameters()
{
	FMemory::Memzero(*this);

	FRHITexture* BlackVolume = (GBlackVolumeTexture &&  GBlackVolumeTexture->TextureRHI) ? GBlackVolumeTexture->TextureRHI : GBlackTexture->TextureRHI;
	FRHITexture* BlackUintVolume = (GBlackUintVolumeTexture &&  GBlackUintVolumeTexture->TextureRHI) ? GBlackUintVolumeTexture->TextureRHI : GBlackTexture->TextureRHI;
	check(GBlackVolumeTexture);

	MaterialTextureBilinearClampedSampler = TStaticSamplerState<SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
	MaterialTextureBilinearWrapedSampler = TStaticSamplerState<SF_Bilinear, AM_Wrap, AM_Wrap, AM_Wrap>::GetRHI();

	VolumetricLightmapIndirectionTexture = BlackUintVolume;
	VolumetricLightmapBrickAmbientVector = BlackVolume;
	VolumetricLightmapBrickSHCoefficients0 = BlackVolume;
	VolumetricLightmapBrickSHCoefficients1 = BlackVolume;
	VolumetricLightmapBrickSHCoefficients2 = BlackVolume;
	VolumetricLightmapBrickSHCoefficients3 = BlackVolume;
	VolumetricLightmapBrickSHCoefficients4 = BlackVolume;
	VolumetricLightmapBrickSHCoefficients5 = BlackVolume;
	SkyBentNormalBrickTexture = BlackVolume;
	DirectionalLightShadowingBrickTexture = BlackVolume;

	VolumetricLightmapBrickAmbientVectorSampler = TStaticSamplerState<SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
	VolumetricLightmapTextureSampler0 = TStaticSamplerState<SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
	VolumetricLightmapTextureSampler1 = TStaticSamplerState<SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
	VolumetricLightmapTextureSampler2 = TStaticSamplerState<SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
	VolumetricLightmapTextureSampler3 = TStaticSamplerState<SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
	VolumetricLightmapTextureSampler4 = TStaticSamplerState<SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
	VolumetricLightmapTextureSampler5 = TStaticSamplerState<SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
	SkyBentNormalTextureSampler = TStaticSamplerState<SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
	DirectionalLightShadowingTextureSampler = TStaticSamplerState<SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();

	AtmosphereTransmittanceTexture = GWhiteTexture->TextureRHI;
	AtmosphereTransmittanceTextureSampler = TStaticSamplerState<SF_Bilinear>::GetRHI();
	AtmosphereIrradianceTexture = GWhiteTexture->TextureRHI;
	AtmosphereIrradianceTextureSampler = TStaticSamplerState<SF_Bilinear>::GetRHI();
	AtmosphereInscatterTexture = BlackVolume;
	AtmosphereInscatterTextureSampler = TStaticSamplerState<SF_Bilinear>::GetRHI();

	PerlinNoiseGradientTexture = GWhiteTexture->TextureRHI;
	PerlinNoiseGradientTextureSampler = TStaticSamplerState<SF_Point, AM_Wrap, AM_Wrap, AM_Wrap>::GetRHI();

	PerlinNoise3DTexture = BlackVolume;
	PerlinNoise3DTextureSampler = TStaticSamplerState<SF_Bilinear, AM_Wrap, AM_Wrap, AM_Wrap>::GetRHI();

	SobolSamplingTexture = GWhiteTexture->TextureRHI;

	GlobalDistanceFieldPageAtlasTexture = BlackVolume;
	GlobalDistanceFieldCoverageAtlasTexture = BlackVolume;
	GlobalDistanceFieldPageTableTexture = BlackUintVolume;
	GlobalDistanceFieldMipTexture = BlackVolume;

	GlobalDistanceFieldPageAtlasTextureSampler = TStaticSamplerState<SF_Trilinear, AM_Wrap, AM_Wrap, AM_Wrap>::GetRHI();
	GlobalDistanceFieldCoverageAtlasTextureSampler = TStaticSamplerState<SF_Trilinear, AM_Wrap, AM_Wrap, AM_Wrap>::GetRHI();
	GlobalDistanceFieldMipTextureSampler = TStaticSamplerState<SF_Trilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();

	SharedPointWrappedSampler = TStaticSamplerState<SF_Point, AM_Wrap, AM_Wrap, AM_Wrap>::GetRHI();
	SharedPointClampedSampler = TStaticSamplerState<SF_Point, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
	SharedBilinearWrappedSampler = TStaticSamplerState<SF_Bilinear, AM_Wrap, AM_Wrap, AM_Wrap>::GetRHI();
	SharedBilinearClampedSampler = TStaticSamplerState<SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
	SharedBilinearAnisoClampedSampler = TStaticSamplerState<SF_AnisotropicLinear, AM_Clamp, AM_Clamp, AM_Clamp, 0, 0>::GetRHI();
	SharedTrilinearWrappedSampler = TStaticSamplerState<SF_Trilinear, AM_Wrap, AM_Wrap, AM_Wrap>::GetRHI();
	SharedTrilinearClampedSampler = TStaticSamplerState<SF_Trilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();

	PreIntegratedBRDF = GWhiteTexture->TextureRHI;
	PreIntegratedBRDFSampler = TStaticSamplerState<SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();

	TransmittanceLutTexture = GWhiteTexture->TextureRHI;
	TransmittanceLutTextureSampler = TStaticSamplerState<SF_Bilinear>::GetRHI();

	SkyViewLutTexture = GBlackTexture->TextureRHI;
	SkyViewLutTextureSampler = TStaticSamplerState<SF_Bilinear>::GetRHI();

	DistantSkyLightLutBufferSRV = GBlackFloat4StructuredBufferWithSRV->ShaderResourceViewRHI;
	MobileDistantSkyLightLutBufferSRV = GBlackFloat4VertexBufferWithSRV->ShaderResourceViewRHI;

	CameraAerialPerspectiveVolume = GBlackAlpha1VolumeTexture->TextureRHI;
	CameraAerialPerspectiveVolumeSampler = TStaticSamplerState<SF_Bilinear>::GetRHI();
	CameraAerialPerspectiveVolumeMieOnly = GBlackAlpha1VolumeTexture->TextureRHI;
	CameraAerialPerspectiveVolumeMieOnlySampler = TStaticSamplerState<SF_Bilinear>::GetRHI();
	CameraAerialPerspectiveVolumeRayOnly = GBlackAlpha1VolumeTexture->TextureRHI;
	CameraAerialPerspectiveVolumeRayOnlySampler = TStaticSamplerState<SF_Bilinear>::GetRHI();

	SkyIrradianceEnvironmentMap = GIdentityPrimitiveBuffer.SkyIrradianceEnvironmentMapSRV;

	// Physics field
	PhysicsFieldClipmapBuffer = GWhiteVertexBufferWithSRV->ShaderResourceViewRHI;
	PhysicsFieldNodesParams = GWhiteVertexBufferWithSRV->ShaderResourceViewRHI;
	PhysicsFieldNodesOffsets = GWhiteVertexBufferWithSRV->ShaderResourceViewRHI;
	PhysicsFieldTargetsOffsets = GWhiteVertexBufferWithSRV->ShaderResourceViewRHI;

	// Water
	WaterIndirection = GWhiteVertexBufferWithSRV->ShaderResourceViewRHI;
	WaterData = GWhiteVertexBufferWithSRV->ShaderResourceViewRHI;

	// Landscape
	LandscapeWeightmapSampler = TStaticSamplerState<SF_AnisotropicPoint, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
	LandscapeIndirection = GWhiteVertexBufferWithSRV->ShaderResourceViewRHI;
	LandscapePerComponentData = GWhiteVertexBufferWithSRV->ShaderResourceViewRHI;

	// Hair
	HairScatteringLUTTexture = BlackVolume;
	HairScatteringLUTSampler = TStaticSamplerState<SF_Bilinear>::GetRHI();

	// GGX/Sheen - Rect area light
	GGXLTCMatTexture = GBlackTextureWithSRV->TextureRHI;
	GGXLTCMatSampler = TStaticSamplerState<SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
	GGXLTCAmpTexture = GBlackTextureWithSRV->TextureRHI;
	GGXLTCAmpSampler = TStaticSamplerState<SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
	SheenLTCTexture = GBlackTextureWithSRV->TextureRHI;
	SheenLTCSampler = TStaticSamplerState<SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();

	// Shading energy conservation
	bShadingEnergyConservation = 0u;
	bShadingEnergyPreservation = 0u;
	ShadingEnergySampler = TStaticSamplerState<SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
	ShadingEnergyGGXSpecTexture = GBlackTextureWithSRV->TextureRHI;
	ShadingEnergyGGXMaxSpecTexture = GWhiteTextureWithSRV->TextureRHI;
	ShadingEnergyGGXGlassTexture = BlackVolume;
	ShadingEnergyClothSpecTexture = GBlackTextureWithSRV->TextureRHI;
	ShadingEnergyDiffuseTexture = GBlackTextureWithSRV->TextureRHI;

	// Glint
	GlintTexture = GBlackArrayTexture->TextureRHI;
	GlintSampler = TStaticSamplerState<SF_Trilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();

	// SimpleVolume
	SimpleVolumeTexture = GBlackVolumeTexture->TextureRHI;
	SimpleVolumeTextureSampler = TStaticSamplerState<SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
	SimpleVolumeEnvTexture = GBlackVolumeTexture->TextureRHI;
	SimpleVolumeEnvTextureSampler = TStaticSamplerState<SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();

	// Rect light atlas
	RectLightAtlasMaxMipLevel = 1;
	RectLightAtlasSizeAndInvSize = FVector4f(1, 1, 1, 1);
	RectLightAtlasTexture = GBlackTextureWithSRV->TextureRHI;
	RectLightAtlasSampler = TStaticSamplerState<SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();

	// IES atlas
	IESAtlasSizeAndInvSize = FVector4f(1, 1, 1, 1);
	IESAtlasTexture = GBlackTextureWithSRV->TextureRHI;
	IESAtlasSampler = TStaticSamplerState<SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();

	// Subsurface profiles
	SSProfilesTextureSizeAndInvSize = FVector4f(1.f,1.f,1.f,1.f);
	SSProfilesTexture = GBlackTextureWithSRV->TextureRHI;
	SSProfilesSampler = TStaticSamplerState<SF_Point, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();;
	SSProfilesTransmissionSampler = TStaticSamplerState<SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();

	// Subsurface pre-intregrated profiles
	SSProfilesPreIntegratedTextureSizeAndInvSize = FVector4f(1.f,1.f,1.f,1.f);
	SSProfilesPreIntegratedTexture = GBlackArrayTexture->TextureRHI;
	SSProfilesPreIntegratedSampler = TStaticSamplerState<SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();

	// Specular profiles
	SpecularProfileTextureSizeAndInvSize = FVector4f(1.f,1.f,1.f,1.f);
	SpecularProfileTexture = GBlackArrayTexture->TextureRHI;
	SpecularProfileSampler = TStaticSamplerState<SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();

	// Toon profiles
	ToonProfileTextureSizeAndInvSize = FVector4f(1.f, 1.f, 1.f, 1.f);
	ToonProfileDitherPatternSizeAndInvSize = FVector2f(1.f, 1.f);
	ToonProfileTexture = GBlackArrayTexture->TextureRHI;
	ToonProfileSampler = TStaticSamplerState<SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();

	VTFeedbackBuffer = GEmptyStructuredBufferWithUAV->UnorderedAccessViewRHI;

	BlueNoiseScalarTexture = GBlackTextureWithSRV->TextureRHI;
}

FInstancedViewUniformShaderParameters::FInstancedViewUniformShaderParameters()
{
	FMemory::Memzero(*this);
}

void FSharedSamplerState::InitRHI(FRHICommandListBase&)
{
	const float MipMapBias = UTexture2D::GetGlobalMipMapLODBias();

	const UTextureLODSettings* TextureLODSettings = UDeviceProfileManager::Get().GetActiveProfile()->GetTextureLODSettings();
	FSamplerStateInitializerRHI SamplerStateInitializer
	(
	(ESamplerFilter)TextureLODSettings->GetSamplerFilter(TEXTUREGROUP_World),
		bWrap ? AM_Wrap : AM_Clamp,
		bWrap ? AM_Wrap : AM_Clamp,
		bWrap ? AM_Wrap : AM_Clamp,
		MipMapBias,
		TextureLODSettings->GetTextureLODGroup(TEXTUREGROUP_World).MaxAniso
	);
	SamplerStateRHI = RHICreateSamplerState(SamplerStateInitializer);
}

FSharedSamplerState* Wrap_WorldGroupSettings = NULL;
FSharedSamplerState* Clamp_WorldGroupSettings = NULL;

void InitializeSharedSamplerStates()
{
	if (!Wrap_WorldGroupSettings && FApp::CanEverRender())
	{
		Wrap_WorldGroupSettings = new FSharedSamplerState(true);
		Clamp_WorldGroupSettings = new FSharedSamplerState(false);
		BeginInitResource(Wrap_WorldGroupSettings);
		BeginInitResource(Clamp_WorldGroupSettings);
	}
}

void FLightCacheInterface::CreatePrecomputedLightingUniformBuffer_RenderingThread(ERHIFeatureLevel::Type FeatureLevel)
{
	const bool bPrecomputedLightingParametersFromGPUScene = UseGPUScene(GShaderPlatformForFeatureLevel[FeatureLevel]) && bCanUsePrecomputedLightingParametersFromGPUScene;

	// Only create UB when GPUScene isn't available
	if (!bPrecomputedLightingParametersFromGPUScene && (LightMap || ShadowMap))
	{
		FPrecomputedLightingUniformParameters Parameters;
		GetPrecomputedLightingParameters(FeatureLevel, Parameters, this);
		if (PrecomputedLightingUniformBuffer)
		{
			// Don't recreate the buffer if it already exists
			FRHICommandListImmediate::Get().UpdateUniformBuffer(PrecomputedLightingUniformBuffer, &Parameters);
		}
		else
		{
			PrecomputedLightingUniformBuffer = FPrecomputedLightingUniformParameters::CreateUniformBuffer(Parameters, UniformBuffer_MultiFrame);
		}
	}
}

bool FLightCacheInterface::GetVirtualTextureLightmapProducer(ERHIFeatureLevel::Type FeatureLevel, FVirtualTextureProducerHandle& OutProducerHandle)
{
	const FLightMapInteraction LightMapInteraction = GetLightMapInteraction(FeatureLevel);
	if (LightMapInteraction.GetType() == LMIT_Texture)
	{
		const ULightMapVirtualTexture2D* VirtualTexture = LightMapInteraction.GetVirtualTexture();
		// Preview lightmaps don't stream from disk, thus no FVirtualTexture2DResource
		if (VirtualTexture && !VirtualTexture->bPreviewLightmap)
		{
			FVirtualTexture2DResource* Resource = (FVirtualTexture2DResource*)VirtualTexture->GetResource();
			OutProducerHandle = Resource->GetProducerHandle();
			return true;
		}
	}
	return false;
}

IMPLEMENT_GLOBAL_SHADER_PARAMETER_STRUCT(FLightmapResourceClusterShaderParameters, "LightmapResourceCluster");

static FRHISamplerState* GetTextureSamplerState(const UTexture* Texture, FRHISamplerState* Default)
{
	FRHISamplerState* Result = nullptr;
	if (Texture && Texture->GetResource())
	{
		Result = Texture->GetResource()->SamplerStateRHI;
	}
	return Result ? Result : Default;
}

void GetLightmapClusterResourceParameters(
	ERHIFeatureLevel::Type FeatureLevel, 
	const FLightmapClusterResourceInput& Input,
	const IAllocatedVirtualTexture* AllocatedVT,
	FLightmapResourceClusterShaderParameters& Parameters)
{
	const bool bAllowHighQualityLightMaps = AllowHighQualityLightmaps(FeatureLevel);

	const bool bUseVirtualTextures = UseVirtualTextureLightmap(GetFeatureLevelShaderPlatform(FeatureLevel));

	Parameters.LightMapTexture = GBlackTexture->TextureRHI;
	Parameters.SkyOcclusionTexture = GWhiteTexture->TextureRHI;
	Parameters.AOMaterialMaskTexture = GBlackTexture->TextureRHI;
	Parameters.StaticShadowTexture = GWhiteTexture->TextureRHI;
	Parameters.VTLightMapTexture = GBlackTextureWithSRV->ShaderResourceViewRHI;
	Parameters.VTLightMapTexture_1 = GBlackTextureWithSRV->ShaderResourceViewRHI;
	Parameters.VTSkyOcclusionTexture = GWhiteTextureWithSRV->ShaderResourceViewRHI;
	Parameters.VTAOMaterialMaskTexture = GBlackTextureWithSRV->ShaderResourceViewRHI;
	Parameters.VTStaticShadowTexture = GWhiteTextureWithSRV->ShaderResourceViewRHI;
	Parameters.LightmapVirtualTexturePageTable0 = GBlackUintTexture->TextureRHI;
	Parameters.LightmapVirtualTexturePageTable1 = GBlackUintTexture->TextureRHI;
	Parameters.LightMapSampler = GBlackTexture->SamplerStateRHI;
	Parameters.LightMapSampler_1 = GBlackTexture->SamplerStateRHI;
	Parameters.SkyOcclusionSampler = GWhiteTexture->SamplerStateRHI;
	Parameters.AOMaterialMaskSampler = GBlackTexture->SamplerStateRHI;
	Parameters.StaticShadowTextureSampler = GWhiteTexture->SamplerStateRHI;

	if (bUseVirtualTextures)
	{
		// this is sometimes called with NULL input to initialize default buffer
		const ULightMapVirtualTexture2D* VirtualTexture = Input.LightMapVirtualTextures[bAllowHighQualityLightMaps ? 0 : 1];
		if (VirtualTexture && AllocatedVT)
		{
			// Bind VT here
			Parameters.VTLightMapTexture = AllocatedVT->GetPhysicalTextureSRV((uint32)ELightMapVirtualTextureType::LightmapLayer0, false);
			Parameters.VTLightMapTexture_1 = AllocatedVT->GetPhysicalTextureSRV((uint32)ELightMapVirtualTextureType::LightmapLayer1, false);

			if (VirtualTexture->HasLayerForType(ELightMapVirtualTextureType::SkyOcclusion))
			{
				Parameters.VTSkyOcclusionTexture = AllocatedVT->GetPhysicalTextureSRV((uint32)ELightMapVirtualTextureType::SkyOcclusion, false);
			}
			else
			{
				Parameters.VTSkyOcclusionTexture = GWhiteTextureWithSRV->ShaderResourceViewRHI;
			}

			if (VirtualTexture->HasLayerForType(ELightMapVirtualTextureType::AOMaterialMask))
			{
				Parameters.VTAOMaterialMaskTexture = AllocatedVT->GetPhysicalTextureSRV((uint32)ELightMapVirtualTextureType::AOMaterialMask, false);
			}
			else
			{
				Parameters.VTAOMaterialMaskTexture = GBlackTextureWithSRV->ShaderResourceViewRHI;
			}

			if (VirtualTexture->HasLayerForType(ELightMapVirtualTextureType::ShadowMask))
			{
				Parameters.VTStaticShadowTexture = AllocatedVT->GetPhysicalTextureSRV((uint32)ELightMapVirtualTextureType::ShadowMask, false);
			}
			else
			{
				Parameters.VTStaticShadowTexture = GWhiteTextureWithSRV->ShaderResourceViewRHI;
			}

			FRHITexture* PageTable0 = AllocatedVT->GetPageTableTexture(0u);
			Parameters.LightmapVirtualTexturePageTable0 = PageTable0;
			if (AllocatedVT->GetNumPageTableTextures() > 1u)
			{
				check(AllocatedVT->GetNumPageTableTextures() == 2u);
				Parameters.LightmapVirtualTexturePageTable1 = AllocatedVT->GetPageTableTexture(1u);
			}
			else
			{
				Parameters.LightmapVirtualTexturePageTable1 = PageTable0;
			}

			const uint32 MaxAniso = 4;
			Parameters.LightMapSampler = TStaticSamplerState<SF_AnisotropicLinear, AM_Clamp, AM_Clamp, AM_Clamp, 0, MaxAniso>::GetRHI();
			Parameters.LightMapSampler_1 = TStaticSamplerState<SF_AnisotropicLinear, AM_Clamp, AM_Clamp, AM_Clamp, 0, MaxAniso>::GetRHI();
			Parameters.SkyOcclusionSampler = TStaticSamplerState<SF_AnisotropicLinear, AM_Clamp, AM_Clamp, AM_Clamp, 0, MaxAniso>::GetRHI();
			Parameters.AOMaterialMaskSampler = TStaticSamplerState<SF_AnisotropicLinear, AM_Clamp, AM_Clamp, AM_Clamp, 0, MaxAniso>::GetRHI();
			Parameters.StaticShadowTextureSampler = TStaticSamplerState<SF_AnisotropicLinear, AM_Clamp, AM_Clamp, AM_Clamp, 0, MaxAniso>::GetRHI();
		}
	}
	else
	{
		const UTexture2D* LightMapTexture = Input.LightMapTextures[bAllowHighQualityLightMaps ? 0 : 1];

		Parameters.LightMapTexture = LightMapTexture ? LightMapTexture->TextureReference.TextureReferenceRHI.GetReference() : GBlackTexture->TextureRHI;
		Parameters.SkyOcclusionTexture = Input.SkyOcclusionTexture ? Input.SkyOcclusionTexture->TextureReference.TextureReferenceRHI.GetReference() : GWhiteTexture->TextureRHI;
		Parameters.AOMaterialMaskTexture = Input.AOMaterialMaskTexture ? Input.AOMaterialMaskTexture->TextureReference.TextureReferenceRHI.GetReference() : GBlackTexture->TextureRHI;

		Parameters.LightMapSampler = GetTextureSamplerState(LightMapTexture, GBlackTexture->SamplerStateRHI);
		Parameters.LightMapSampler_1 = GetTextureSamplerState(LightMapTexture, GBlackTexture->SamplerStateRHI);
		Parameters.SkyOcclusionSampler = GetTextureSamplerState(Input.SkyOcclusionTexture, GWhiteTexture->SamplerStateRHI);
		Parameters.AOMaterialMaskSampler = GetTextureSamplerState(Input.AOMaterialMaskTexture, GBlackTexture->SamplerStateRHI);

		Parameters.StaticShadowTexture = Input.ShadowMapTexture ? Input.ShadowMapTexture->TextureReference.TextureReferenceRHI.GetReference() : GWhiteTexture->TextureRHI;
		Parameters.StaticShadowTextureSampler = GetTextureSamplerState(Input.ShadowMapTexture, GWhiteTexture->SamplerStateRHI);

		Parameters.LightmapVirtualTexturePageTable0 = GBlackUintTexture->TextureRHI;
		Parameters.LightmapVirtualTexturePageTable1 = GBlackUintTexture->TextureRHI;
	}
}

void FDefaultLightmapResourceClusterUniformBuffer::InitRHI(FRHICommandListBase& RHICmdList)
{
	FLightmapResourceClusterShaderParameters Parameters;
	GetLightmapClusterResourceParameters(GMaxRHIFeatureLevel, FLightmapClusterResourceInput(), nullptr, Parameters);
	SetContentsNoUpdate(Parameters);
	Super::InitRHI(RHICmdList);
}

/** Global uniform buffer containing the default precomputed lighting data. */
TGlobalResource< FDefaultLightmapResourceClusterUniformBuffer > GDefaultLightmapResourceClusterUniformBuffer;

FLightMapInteraction FLightCacheInterface::GetLightMapInteraction(ERHIFeatureLevel::Type InFeatureLevel) const
{
	if (bGlobalVolumeLightmap)
	{
		return FLightMapInteraction::GlobalVolume();
	}

	return LightMap ? LightMap->GetInteraction(InFeatureLevel) : FLightMapInteraction();
}

FShadowMapInteraction FLightCacheInterface::GetShadowMapInteraction(ERHIFeatureLevel::Type InFeatureLevel) const
{
	if (bGlobalVolumeLightmap)
	{
		return FShadowMapInteraction::GlobalVolume();
	}

	FShadowMapInteraction Interaction;
	if (LightMap)
	{
		// Lightmap gets the first chance to provide shadow interaction,
		// this is used if VT lightmaps are enabled, and shadowmap is packed into the same VT stack as other lightmap textures
		Interaction = LightMap->GetShadowInteraction(InFeatureLevel);
	}
	if (Interaction.GetType() == SMIT_None && ShadowMap)
	{
		Interaction = ShadowMap->GetInteraction();
	}

	return Interaction;
}

ELightInteractionType FLightCacheInterface::GetStaticInteraction(const FLightSceneProxy* LightSceneProxy, const TArray<FGuid>& IrrelevantLights) const
{
	if (bGlobalVolumeLightmap)
	{
		if (LightSceneProxy->HasStaticLighting())
		{
			return LIT_CachedLightMap;
		}
		else if (LightSceneProxy->HasStaticShadowing())
		{
			return LIT_CachedSignedDistanceFieldShadowMap2D;
		}
		else
		{
			return LIT_MAX;
		}
	}

	ELightInteractionType Ret = LIT_MAX;

	// Check if the light has static lighting or shadowing.
	if(LightSceneProxy->HasStaticShadowing())
	{
		const FGuid LightGuid = LightSceneProxy->GetLightGuid();

		if(IrrelevantLights.Contains(LightGuid))
		{
			Ret = LIT_CachedIrrelevant;
		}
		else if(LightMap && LightMap->ContainsLight(LightGuid))
		{
			Ret = LIT_CachedLightMap;
		}
		else if(ShadowMap && ShadowMap->ContainsLight(LightGuid))
		{
			Ret = LIT_CachedSignedDistanceFieldShadowMap2D;
		}
	}

	return Ret;
}

void FMeshBatch::PreparePrimitiveUniformBuffer(const FPrimitiveSceneProxy* PrimitiveSceneProxy, ERHIFeatureLevel::Type FeatureLevel)
{
	// Fallback to using the primitive uniform buffer if GPU scene is disabled.
	// Vertex shaders on mobile may still use PrimitiveUB with GPUScene enabled
	if (!UseGPUScene(GShaderPlatformForFeatureLevel[FeatureLevel]) || FeatureLevel == ERHIFeatureLevel::ES3_1)
	{
		for (int32 ElementIndex = 0; ElementIndex < Elements.Num(); ElementIndex++)
		{
			FMeshBatchElement& MeshElement = Elements[ElementIndex];

			if (!MeshElement.PrimitiveUniformBuffer && !MeshElement.PrimitiveUniformBufferResource)
			{
				MeshElement.PrimitiveUniformBuffer = PrimitiveSceneProxy->GetUniformBuffer();
			}
		}
	}
}

#if USE_MESH_BATCH_VALIDATION
bool FMeshBatch::Validate(const FPrimitiveSceneProxy* PrimitiveSceneProxy, ERHIFeatureLevel::Type FeatureLevel) const
{
	check(PrimitiveSceneProxy);

	const auto LogMeshError = [&](const FString& Error) -> bool
	{
		const FString VertexFactoryName = VertexFactory ? VertexFactory->GetType()->GetFName().ToString() : TEXT("nullptr");
		const uint32 bVertexFactoryInitialized = (VertexFactory && VertexFactory->IsInitialized()) ? 1 : 0;

		ensureMsgf(false,
			TEXT("FMeshBatch was not properly setup. %s.\n\tVertexFactory[Name: %s, Initialized: %u]\n\tPrimitiveSceneProxy[Level: %s, Owner: %s, Resource: %s]"),
			*Error,
			*VertexFactoryName,
			bVertexFactoryInitialized,
			*PrimitiveSceneProxy->GetLevelName().ToString(),
			*PrimitiveSceneProxy->GetOwnerName().ToString(),
			*PrimitiveSceneProxy->GetResourceName().ToString());

		return false;
	};

	if (!MaterialRenderProxy)
	{
		return LogMeshError(TEXT("Mesh has a null material render proxy!"));
	}

	if (!PrimitiveSceneProxy->VerifyUsedMaterial(MaterialRenderProxy))
	{
		return LogMeshError(TEXT("Mesh material is not marked as used by the primitive scene proxy."));
	}

	if (!VertexFactory)
	{
		return LogMeshError(TEXT("Mesh has a null vertex factory!"));
}

	if (!VertexFactory->IsInitialized())
	{
		return LogMeshError(TEXT("Mesh has an uninitialized vertex factory!"));
}

	for (int32 Index = 0; Index < Elements.Num(); ++Index)
	{
		const FMeshBatchElement& MeshBatchElement = Elements[Index];

		if (MeshBatchElement.IndexBuffer)
		{
			if (const FRHIBuffer* IndexBufferRHI = MeshBatchElement.IndexBuffer->IndexBufferRHI)
			{
				const uint32 IndexCount = GetVertexCountForPrimitiveCount(MeshBatchElement.NumPrimitives, Type);
				const uint32 IndexBufferSize = IndexBufferRHI->GetSize();

				// A zero-sized index buffer is valid for streaming.
				if (IndexBufferSize != 0 && (MeshBatchElement.FirstIndex + IndexCount) * IndexBufferRHI->GetStride() > IndexBufferSize)
				{
					return LogMeshError(FString::Printf(
						TEXT("MeshBatchElement %d, Material '%s', index range extends past index buffer bounds: Start %u, Count %u, Buffer Size %u, Buffer stride %u"),
						Index, MaterialRenderProxy ? *MaterialRenderProxy->GetFriendlyName() : TEXT("nullptr"),
						MeshBatchElement.FirstIndex, IndexCount, IndexBufferRHI->GetSize(), IndexBufferRHI->GetStride()));
				}
			}
			else
			{
				return LogMeshError(FString::Printf(
					TEXT("FMeshElementCollector::AddMesh - On MeshBatchElement %d, Material '%s', index buffer object has null RHI resource"),
					Index, MaterialRenderProxy ? *MaterialRenderProxy->GetFriendlyName() : TEXT("nullptr")));
			}
		}
	}

	const bool bVFSupportsPrimitiveIdStream = VertexFactory->GetType()->SupportsPrimitiveIdStream();
	const bool bVFRequiresPrimitiveUniformBuffer = PrimitiveSceneProxy->DoesVFRequirePrimitiveUniformBuffer();

	if (!bVFRequiresPrimitiveUniformBuffer && !bVFSupportsPrimitiveIdStream)
	{
		return LogMeshError(TEXT("PrimitiveSceneProxy has bVFRequiresPrimitiveUniformBuffer disabled yet tried to draw with a vertex factory that did not support PrimitiveIdStream"));
	}

	// Some primitives may use several VFs with a mixed support for a GPUScene
	if (PrimitiveSceneProxy->SupportsGPUScene() && !(VertexFactory->SupportsGPUScene(FeatureLevel) || bVFRequiresPrimitiveUniformBuffer))
	{
		return LogMeshError(TEXT("PrimitiveSceneProxy has SupportsGPUScene() does not match VertexFactory->SupportsGPUScene() or bVFRequiresPrimitiveUniformBuffer"));
	}
	const bool bUseGPUScene = UseGPUScene(GShaderPlatformForFeatureLevel[FeatureLevel]);
	
	const bool bPrimitiveShaderDataComesFromSceneBuffer = bUseGPUScene && VertexFactory->GetPrimitiveIdStreamIndex(FeatureLevel, EVertexInputStreamType::Default) >= 0;

	const bool bPrimitiveHasUniformBuffer = PrimitiveSceneProxy->GetUniformBuffer() != nullptr;

	for (int32 ElementIndex = 0; ElementIndex < Elements.Num(); ElementIndex++)
	{
		const FMeshBatchElement& MeshElement = Elements[ElementIndex];

		// Some primitives may use several VFs with a mixed support for a GPUScene 
		// in this case all mesh batches get Primitive UB assigned regardless of VF type
		if (bPrimitiveShaderDataComesFromSceneBuffer && Elements[ElementIndex].PrimitiveUniformBuffer && !bVFRequiresPrimitiveUniformBuffer)
		{
			// on mobile VS has access to PrimitiveUniformBuffer
			if (FeatureLevel > ERHIFeatureLevel::ES3_1)
			{
				// This is a non-fatal error.
				LogMeshError(
					TEXT("FMeshBatch was assigned a PrimitiveUniformBuffer even though the vertex factory fetches primitive shader data through the GPUScene buffer. ")
					TEXT("The assigned PrimitiveUniformBuffer cannot be respected. Use PrimitiveUniformBufferResource instead for dynamic primitive data, or leave ")
					TEXT("both null to get FPrimitiveSceneProxy->UniformBuffer"));
			}
		}

		const bool bValidPrimitiveData =
			   bPrimitiveShaderDataComesFromSceneBuffer
			|| bPrimitiveHasUniformBuffer
			|| Elements[ElementIndex].PrimitiveUniformBuffer
			|| Elements[ElementIndex].PrimitiveUniformBufferResource;

		if (!bValidPrimitiveData)
		{
			return LogMeshError(TEXT("No primitive uniform buffer was specified and the vertex factory does not have a valid primitive id stream"));
		}
	}

	return true;
}
#endif

IMPLEMENT_GLOBAL_SHADER_PARAMETER_STRUCT(FMobileReflectionCaptureShaderParameters, "MobileReflectionCapture");

void FDefaultMobileReflectionCaptureUniformBuffer::InitRHI(FRHICommandListBase& RHICmdList)
{
	FMobileReflectionCaptureShaderParameters Parameters;
	Parameters.Params = FVector4f(1.f, 0.f, 0.f, 0.f);
	Parameters.Texture = GBlackTextureCube->TextureRHI;
	Parameters.TextureSampler = GBlackTextureCube->SamplerStateRHI;
	Parameters.TextureBlend = Parameters.Texture;
	Parameters.TextureBlendSampler = Parameters.TextureSampler;
	SetContentsNoUpdate(Parameters);
	Super::InitRHI(RHICmdList);
}

/** Global uniform buffer containing the default reflection data used in mobile renderer. */
TGlobalResource<FDefaultMobileReflectionCaptureUniformBuffer> GDefaultMobileReflectionCaptureUniformBuffer;
