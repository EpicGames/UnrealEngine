// Copyright Epic Games, Inc. All Rights Reserved.

#include "SceneRelativeViewMatrices.h"
#include "Misc/LargeWorldRenderPosition.h"
#include "SceneView.h"
#include "Math/DoubleFloat.h"

FRelativeViewMatrices FRelativeViewMatrices::Create(const FViewMatrices& Matrices, const FViewMatrices& PrevMatrices)
{
	FInitializer Initializer;
	Initializer.ViewToWorld = Matrices.GetViewToWorld();
	Initializer.WorldToView = Matrices.GetWorldToView();
	Initializer.ViewToClip = Matrices.GetViewToClip();
	Initializer.ClipToView = Matrices.GetClipToView();
	Initializer.PrevViewToWorld = PrevMatrices.GetViewToWorld();
	Initializer.PrevClipToView = PrevMatrices.GetClipToView();
	return Create(Initializer);
}

FRelativeViewMatrices FRelativeViewMatrices::Create(const FInitializer& Initializer)
{
	const double TileSize = FLargeWorldRenderScalar::GetTileSize();

	// We allow a fractional tile position here
	// Tile offset is applied beofre WorldToView transform, but after ViewToWorld transform
	// This means that if we use a regular tile offset, the remaining offset in the relative matrices may become too large (>TileSize/2)
	// Allowing for a fractional tile lets us use the same value for both matrices
	// Quantization factor is somewhat arbitrary...controls distribution of precision between tile fraction and relative offset
	const FVector ViewOrigin = Initializer.ViewToWorld.GetOrigin();
	const FVector ViewOriginTile = FLargeWorldRenderScalar::MakeQuantizedTile(ViewOrigin, 8.0);

	FRelativeViewMatrices Result;
	Result.TilePosition = (FVector3f)ViewOriginTile;
	Result.RelativeWorldToView = FLargeWorldRenderScalar::MakeFromRelativeWorldMatrix(ViewOriginTile * TileSize, Initializer.WorldToView);
	Result.ViewToRelativeWorld = FLargeWorldRenderScalar::MakeToRelativeWorldMatrix(ViewOriginTile * TileSize, Initializer.ViewToWorld);
	Result.ViewToClip = FMatrix44f(Initializer.ViewToClip);
	Result.ClipToView = FMatrix44f(Initializer.ClipToView);
	Result.RelativeWorldToClip = Result.RelativeWorldToView * Result.ViewToClip;
	Result.ClipToRelativeWorld = Result.ClipToView * Result.ViewToRelativeWorld;

	Result.PrevViewToRelativeWorld = FLargeWorldRenderScalar::MakeClampedToRelativeWorldMatrix(ViewOriginTile * TileSize, Initializer.PrevViewToWorld);
	Result.PrevClipToView = FMatrix44f(Initializer.PrevClipToView);
	Result.PrevClipToRelativeWorld = Result.PrevClipToView * Result.PrevViewToRelativeWorld;
	return Result;
}


FDFRelativeViewMatrices FDFRelativeViewMatrices::Create(const FViewMatrices& Matrices, const FViewMatrices& PrevMatrices)
{
	FInitializer Initializer;
	Initializer.ViewOrigin = Matrices.GetViewOrigin();
	Initializer.ViewToWorld = Matrices.GetViewToWorld();
	Initializer.WorldToView = Matrices.GetWorldToView();
	Initializer.ViewToClip = Matrices.GetViewToClip();
	Initializer.ClipToView = Matrices.GetClipToView();
	Initializer.PrevViewToWorld = PrevMatrices.GetViewToWorld();
	Initializer.PrevClipToView = PrevMatrices.GetClipToView();
	return Create(Initializer);
}

FDFRelativeViewMatrices FDFRelativeViewMatrices::Create(const FInitializer& Initializer)
{
	FDFVector3 ViewOrigin(Initializer.ViewOrigin);

	FDFRelativeViewMatrices Result;
	Result.PositionHigh = ViewOrigin.High;
	Result.RelativeWorldToView = FDFInverseMatrix::MakeFromRelativeWorldMatrix(ViewOrigin.High, Initializer.WorldToView).M;
	Result.ViewToRelativeWorld = FDFMatrix::MakeToRelativeWorldMatrix(ViewOrigin.High, Initializer.ViewToWorld).M;
	Result.ViewToClip = FMatrix44f(Initializer.ViewToClip);
	Result.ClipToView = FMatrix44f(Initializer.ClipToView);
	Result.RelativeWorldToClip = Result.RelativeWorldToView * Result.ViewToClip;
	Result.ClipToRelativeWorld = Result.ClipToView * Result.ViewToRelativeWorld;

	Result.PrevViewToRelativeWorld = FDFMatrix::MakeClampedToRelativeWorldMatrix(ViewOrigin.High, Initializer.PrevViewToWorld).M;
	Result.PrevClipToView = FMatrix44f(Initializer.PrevClipToView);
	Result.PrevClipToRelativeWorld = Result.PrevClipToView * Result.PrevViewToRelativeWorld;
	return Result;
}
