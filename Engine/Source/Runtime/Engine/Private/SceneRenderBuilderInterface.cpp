// Copyright Epic Games, Inc. All Rights Reserved.

#include "SceneRenderBuilderInterface.h"
#include "SceneRendererInterface.h"
#include "EngineModule.h"
#include "RendererInterface.h"

TUniquePtr<ISceneRenderBuilder> ISceneRenderBuilder::Create(FSceneInterface* SceneInterface)
{
	return GetRendererModule().CreateSceneRenderBuilder(SceneInterface);
}