// Copyright Epic Games, Inc. All Rights Reserved.

#include "SceneTexturesConfig.h"
#include "ShaderCompiler.h"
#include "Engine/Engine.h"
#include "RenderGraphResources.h"
#include "RenderUtils.h"
#include "SceneInterface.h"
#include "StereoRenderTargetManager.h"
#include "DataDrivenShaderPlatformInfo.h"

FSceneTexturesConfig FSceneTexturesConfig::GlobalInstance;

DEFINE_LOG_CATEGORY_STATIC(LogSceneTextures, Log, All);
IMPLEMENT_STATIC_UNIFORM_BUFFER_SLOT(SceneTextures);
IMPLEMENT_STATIC_UNIFORM_BUFFER_STRUCT(FSceneTextureUniformParameters, "SceneTexturesStruct", SceneTextures);
IMPLEMENT_STATIC_UNIFORM_BUFFER_STRUCT(FMobileSceneTextureUniformParameters, "MobileSceneTextures", SceneTextures);

FSceneTextureShaderParameters GetSceneTextureShaderParameters(TRDGUniformBufferRef<FSceneTextureUniformParameters> UniformBuffer)
{
	FSceneTextureShaderParameters Parameters;
	Parameters.SceneTextures = UniformBuffer;
	return Parameters;
}

FSceneTextureShaderParameters GetSceneTextureShaderParameters(TRDGUniformBufferRef<FMobileSceneTextureUniformParameters> UniformBuffer)
{
	FSceneTextureShaderParameters Parameters;
	Parameters.MobileSceneTextures = UniformBuffer;
	return Parameters;
}

static EPixelFormat GetDefaultMobileSceneColorLowPrecisionFormat(bool bRequiresAlphaChannel)
{
	bool bShouldUseRGBRT = !bRequiresAlphaChannel && GPixelFormats[PF_R8G8B8].Supported && !!(GPixelFormats[PF_R8G8B8].Capabilities & EPixelFormatCapabilities::RenderTarget);

	if (UNLIKELY(GEngine && GEngine->XRSystem.IsValid() && GEngine->StereoRenderingDevice.IsValid()))
	{
		IStereoRenderTargetManager* RTManager = GEngine->StereoRenderingDevice->GetRenderTargetManager();
		if (LIKELY(RTManager))
		{
			EPixelFormat ActualFormat = RTManager->GetActualColorSwapchainFormat();
			if (LIKELY(ActualFormat != PF_Unknown))
			{
				return ActualFormat;
			}
		}
		
		if (GEngine->StereoRenderingDevice->IsStandaloneStereoOnlyDevice())
		{
			return bShouldUseRGBRT ? PF_R8G8B8 : PF_R8G8B8A8;
		}
	}
	return  bShouldUseRGBRT ? PF_R8G8B8 : PF_B8G8R8A8;
}

static EPixelFormat GetMobileSceneColorFormat(bool bRequiresAlphaChannel)
{
	EPixelFormat DefaultColorFormat;
	const bool bUseLowPrecisionFormat = !IsMobileHDR() || !GSupportsRenderTargetFormat_PF_FloatRGBA;
	if (bUseLowPrecisionFormat)
	{
		DefaultColorFormat = GetDefaultMobileSceneColorLowPrecisionFormat(bRequiresAlphaChannel);
	}
	else
	{
		DefaultColorFormat = bRequiresAlphaChannel ? PF_FloatRGBA : PF_FloatR11G11B10;
	}

	check(GPixelFormats[DefaultColorFormat].Supported);

	EPixelFormat Format = DefaultColorFormat;
	static const auto CVar = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.Mobile.SceneColorFormat"));
	int32 MobileSceneColor = CVar->GetValueOnAnyThread();
	switch (MobileSceneColor)
	{
	case 1:
		Format = PF_FloatRGBA; break;
	case 2:
		Format = PF_FloatR11G11B10; break;
	case 3:
		if (bUseLowPrecisionFormat)
		{
			// if bUseLowPrecisionFormat, DefaultColorFormat already contains the value of GetDefaultMobileSceneColorLowPrecisionFormat
			checkSlow(DefaultColorFormat == GetDefaultMobileSceneColorLowPrecisionFormat(bRequiresAlphaChannel));
			Format = DefaultColorFormat;
		}
		else
		{
			Format = GetDefaultMobileSceneColorLowPrecisionFormat(bRequiresAlphaChannel);
		}
		break;
	default:
		break;
	}

	return GPixelFormats[Format].Supported ? Format : DefaultColorFormat;
}

static EPixelFormat GetSceneColorFormat(bool bRequiresAlphaChannel)
{
	EPixelFormat Format = PF_FloatRGBA;

	static const auto CVar = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.SceneColorFormat"));

	switch (CVar->GetValueOnAnyThread())
	{
	case 0:
		Format = PF_R8G8B8A8; break;
	case 1:
		Format = PF_A2B10G10R10; break;
	case 2:
		Format = PF_FloatR11G11B10; break;
	case 3:
		Format = PF_FloatRGB; break;
	case 4:
		// default
		break;
	case 5:
		Format = PF_A32B32G32R32F; break;
	}

	// Fallback in case the scene color selected isn't supported.
	if (!GPixelFormats[Format].Supported)
	{
		Format = PF_FloatRGBA;
	}

	if (bRequiresAlphaChannel && !(Format == PF_FloatRGBA || Format == PF_A32B32G32R32F))
	{
		UE_CALL_ONCE([]()
			{
				UE_LOGF(LogSceneTextures, Display, "Enforcing FloatRGBA scene color format due to alpha channel requirement.");
			}
		);
		Format = PF_FloatRGBA;
	}

	return Format;
}

void GetSceneColorFormatAndCreateFlags(ERHIFeatureLevel::Type FeatureLevel, bool bRequiresAlphaChannel, ETextureCreateFlags ExtraSceneColorCreateFlags, uint32 NumSamples, bool bMemorylessMSAA, EPixelFormat& SceneColorFormat, ETextureCreateFlags& SceneColorCreateFlags)
{
	EShadingPath ShadingPath = GetFeatureLevelShadingPath(FeatureLevel);
	switch (ShadingPath)
	{
	case EShadingPath::Deferred:
	{
		SceneColorFormat = GetSceneColorFormat(bRequiresAlphaChannel);
		break;
	}

	case EShadingPath::Mobile:
	{
		SceneColorFormat = GetMobileSceneColorFormat(bRequiresAlphaChannel);
		break;
	}

	default:
		checkNoEntry();
	}

	const bool bIsMobilePlatform = ShadingPath == EShadingPath::Mobile;

	SceneColorCreateFlags = TexCreate_RenderTargetable | TexCreate_ShaderResource | ExtraSceneColorCreateFlags;
	const ETextureCreateFlags sRGBFlag = (bIsMobilePlatform && IsMobileColorsRGB()) ? TexCreate_SRGB : TexCreate_None;
	if (FeatureLevel >= ERHIFeatureLevel::SM5 && NumSamples == 1)
	{
		SceneColorCreateFlags |= TexCreate_UAV;
	}
	if (NumSamples > 1 && bMemorylessMSAA)
	{
		SceneColorCreateFlags |= TexCreate_Memoryless;
	}
	SceneColorCreateFlags |= sRGBFlag;
}

static ETextureCreateFlags GetSceneDepthStencilCreateFlags(uint32 NumSamples, bool bKeepDepthContent, bool bMemorylessMSAA, EShaderPlatform ShaderPlatform, ETextureCreateFlags ExtraSceneDepthCreateFlags)
{
	ETextureCreateFlags DepthCreateFlags = TexCreate_DepthStencilTargetable | TexCreate_ShaderResource | TexCreate_InputAttachmentRead | ExtraSceneDepthCreateFlags;
	// We can't discard the SceneDepth MSAA target if full depth prepass is enabled because it will be used in the base pass later on.
	if (!bKeepDepthContent || (NumSamples > 1 && bMemorylessMSAA && (!IsMobilePlatform(ShaderPlatform) || !MobileUsesFullDepthPrepass(ShaderPlatform))))
	{
		DepthCreateFlags |= TexCreate_Memoryless;
	}
	if (NumSamples == 1 && GRHISupportsDepthUAV)
	{
		DepthCreateFlags |= TexCreate_UAV;
	}
	return DepthCreateFlags;
}

uint32 FSceneTexturesConfig::GetEditorPrimitiveNumSamples(ERHIFeatureLevel::Type FeatureLevel)
{
	uint32 SampleCount = 1;

	if (FeatureLevel >= ERHIFeatureLevel::SM5 && GRHISupportsMSAADepthSampleAccess)
	{
		static const auto CVar = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.MSAA.CompositingSampleCount"));

		SampleCount = CVar->GetValueOnAnyThread();

		if (SampleCount <= 1)
		{
			SampleCount = 1;
		}
		else if (SampleCount <= 2)
		{
			SampleCount = 2;
		}
		else if (SampleCount <= 4)
		{
			SampleCount = 4;
		}
		else
		{
			SampleCount = 8;
		}
	}

	return SampleCount;
}

static void SetupMobileGBufferFlags(FGBufferBindings GBufferBindings[GBL_Num], bool bRequiresMultiPass)
{
	ETextureCreateFlags AddFlags = TexCreate_InputAttachmentRead;
	ETextureCreateFlags RemoveFlags = TexCreate_None;

	if (!bRequiresMultiPass)
	{
		// use memoryless GBuffer when possible
		AddFlags |= TexCreate_Memoryless;
		// memoryless GBuffer cant be used with compute
		RemoveFlags |= TexCreate_UAV;
	}
	else
	{
		RemoveFlags |= TexCreate_Memoryless;
	}

	for (uint32 Layout = 0; Layout < GBL_Num; ++Layout)
	{
		FGBufferBindings& Bindings = GBufferBindings[Layout];

		Bindings.GBufferA.Flags |= AddFlags;
		Bindings.GBufferB.Flags |= AddFlags;
		Bindings.GBufferC.Flags |= AddFlags;
		Bindings.GBufferD.Flags |= AddFlags;
		Bindings.GBufferE.Flags |= AddFlags;
		
		Bindings.GBufferA.Flags &= (~RemoveFlags);
		Bindings.GBufferB.Flags &= (~RemoveFlags);
		Bindings.GBufferC.Flags &= (~RemoveFlags);
		Bindings.GBufferD.Flags &= (~RemoveFlags);
		Bindings.GBufferE.Flags &= (~RemoveFlags);

		// Input attachments with PF_R8G8B8A8 has better support on mobile than PF_B8G8R8A8
		auto OverrideB8G8R8A8 = [](FGBufferBinding& Binding) { if (Binding.Format == PF_B8G8R8A8) Binding.Format = PF_R8G8B8A8; };
		OverrideB8G8R8A8(Bindings.GBufferA);
		OverrideB8G8R8A8(Bindings.GBufferB);
		OverrideB8G8R8A8(Bindings.GBufferC);
		OverrideB8G8R8A8(Bindings.GBufferD);
		OverrideB8G8R8A8(Bindings.GBufferE);
	}
}

static bool MobileRequiresPreciseSceneDepthAux(EShaderPlatform ShaderPlatform)
{
	static const auto CVar = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.Mobile.SceneDepthAux"));
	if (IsMobileDeferredShadingEnabled(ShaderPlatform) || CVar->GetValueOnAnyThread() == 2)
	{
		return true;
	}
	return false;
}

void FSceneTexturesConfig::Init(const FSceneTexturesConfigInitSettings& InitSettings)
{
	FeatureLevel			= InitSettings.FeatureLevel;
	ShadingPath				= GetFeatureLevelShadingPath(FeatureLevel);
	ShaderPlatform			= GetFeatureLevelShaderPlatform(FeatureLevel);
	Extent					= InitSettings.Extent;

	// Note that the GetDefaultMSAACount call is mirrored in the FSceneRenderer constructor where custom render passes are initialized.
	// Custom render passes don't support MSAA, and use a different config with MSAA disabled, when its enabled for the main view.  If
	// this calculation of NumSamples changes, that corresponding code would need to be updated.
	NumSamples				= GetDefaultMSAACount(FeatureLevel, GDynamicRHI->RHIGetPlatformTextureMaxSampleCount());

	EditorPrimitiveNumSamples = GetEditorPrimitiveNumSamples(FeatureLevel);
	ColorFormat				= PF_Unknown;
	ColorClearValue			= FClearValueBinding::Black;
	DepthClearValue			= FClearValueBinding::DepthFar;
	bRequiresAlphaChannel	= InitSettings.bRequiresAlphaChannel;
	bRequireMultiView		= InitSettings.bRequireMultiView;
	bIsUsingGBuffers		= IsUsingGBuffers(ShaderPlatform);
	bSupportsXRTargetManagerDepthAlloc = InitSettings.bSupportsXRTargetManagerDepthAlloc;
    ExtraSceneColorCreateFlags = InitSettings.ExtraSceneColorCreateFlags;
    ExtraSceneDepthCreateFlags = InitSettings.ExtraSceneDepthCreateFlags;
    
    BuildSceneColorAndDepthFlags();

	if (bIsUsingGBuffers)
	{
		FGBufferParams DefaultParams = FShaderCompileUtilities::FetchGBufferParamsRuntime(ShaderPlatform, GBL_Default);

		// GBuffer configuration information is expensive to compute, the results are cached between runs.
		struct FGBufferBindingCache
		{
			FGBufferParams GBufferParams;
			FGBufferBindings Bindings[GBL_Num];
			bool bInitialized = false;
		};
		static FGBufferBindingCache BindingCache;

		if (!BindingCache.bInitialized || BindingCache.GBufferParams != DefaultParams)
		{
			for (uint32 Layout = 0; Layout < GBL_Num; ++Layout)
			{
				GBufferParams[Layout] = (Layout == GBL_Default) ? DefaultParams : FShaderCompileUtilities::FetchGBufferParamsRuntime(ShaderPlatform, (EGBufferLayout)Layout);
				const FGBufferInfo GBufferInfo = FetchFullGBufferInfo(GBufferParams[Layout]);

				BindingCache.Bindings[Layout].GBufferA = FindGBufferBindingByName(GBufferInfo, TEXT("GBufferA"), ShaderPlatform);
				BindingCache.Bindings[Layout].GBufferB = FindGBufferBindingByName(GBufferInfo, TEXT("GBufferB"), ShaderPlatform);
				BindingCache.Bindings[Layout].GBufferC = FindGBufferBindingByName(GBufferInfo, TEXT("GBufferC"), ShaderPlatform);
				BindingCache.Bindings[Layout].GBufferD = FindGBufferBindingByName(GBufferInfo, TEXT("GBufferD"), ShaderPlatform);
				BindingCache.Bindings[Layout].GBufferE = FindGBufferBindingByName(GBufferInfo, TEXT("GBufferE"), ShaderPlatform);
				BindingCache.Bindings[Layout].GBufferVelocity = FindGBufferBindingByName(GBufferInfo, TEXT("Velocity"), ShaderPlatform);

				// Remove DisableDCC flag for velocity. Only Nanite fast tile clear sets this flag currently
				// but we want to exclude velocity because it usually doesn't have many written pixels
				EnumRemoveFlags(BindingCache.Bindings[Layout].GBufferVelocity.Flags, TexCreate_DisableDCC);
			}

			BindingCache.GBufferParams = DefaultParams;
			BindingCache.bInitialized = true;

			if (ShadingPath == EShadingPath::Mobile)
			{
				bool bRequiresMultiPass = false; // This will be overriden later when in the renderer InitViews. Does not affect PSO state
				::SetupMobileGBufferFlags(BindingCache.Bindings, bRequiresMultiPass);
			}
		}

		for (uint32 Layout = 0; Layout < GBL_Num; ++Layout)
		{
			GBufferBindings[Layout] = BindingCache.Bindings[Layout];
		}
	}

	if (ShadingPath == EShadingPath::Mobile)
	{
		bRequiresDepthAux = MobileRequiresSceneDepthAux(ShaderPlatform) && !IsMobileTonemapSubpassEnabled(ShaderPlatform, InitSettings.bRequireMultiView);
		bPreciseDepthAux = bPreciseDepthAux || MobileRequiresPreciseSceneDepthAux(ShaderPlatform);
		bCustomResolveSubpass = IsMobileTonemapSubpassEnabledInline(ShaderPlatform, InitSettings.bRequireMultiView, NumSamples);
	}
}

void FSceneTexturesConfig::SetupMobileGBufferFlags(bool bRequiresMultiPass)
{
	::SetupMobileGBufferFlags(GBufferBindings, bRequiresMultiPass);
}

void FSceneTexturesConfig::BuildSceneColorAndDepthFlags()
{
    GetSceneColorFormatAndCreateFlags(FeatureLevel, bRequiresAlphaChannel, ExtraSceneColorCreateFlags, NumSamples, bMemorylessMSAA, ColorFormat, ColorCreateFlags);
    DepthCreateFlags = GetSceneDepthStencilCreateFlags(NumSamples, bKeepDepthContent, bMemorylessMSAA, ShaderPlatform, ExtraSceneDepthCreateFlags);
}

uint32 FSceneTexturesConfig::GetGBufferRenderTargetsInfo(FGraphicsPipelineRenderTargetsInfo& RenderTargetsInfo, EGBufferLayout Layout) const 
{
	RenderTargetsInfo.NumSamples = NumSamples;

	uint32 RenderTargetCount = 0;

	// All configurations use scene color in the first slot.
	RenderTargetsInfo.RenderTargetFormats[RenderTargetCount] = ColorFormat;
	RenderTargetsInfo.RenderTargetFlags[RenderTargetCount++] = ColorCreateFlags;

	// Setup the other render targets
	if (bIsUsingGBuffers)
	{
		const auto IncludeBindingIfValid = [&RenderTargetsInfo, &RenderTargetCount](const FGBufferBinding& GBufferBinding)
		{
			if (GBufferBinding.Index > 0)
			{
				RenderTargetsInfo.RenderTargetFormats[GBufferBinding.Index] = GBufferBinding.Format;
				RenderTargetsInfo.RenderTargetFlags[GBufferBinding.Index] = GBufferBinding.Flags;
				RenderTargetCount = FMath::Max(RenderTargetCount, (uint32)GBufferBinding.Index + 1);
			}
		};

		const FGBufferBindings& Bindings = GBufferBindings[Layout];
		IncludeBindingIfValid(Bindings.GBufferA);
		IncludeBindingIfValid(Bindings.GBufferB);
		IncludeBindingIfValid(Bindings.GBufferC);
		IncludeBindingIfValid(Bindings.GBufferD);
		IncludeBindingIfValid(Bindings.GBufferE);
		IncludeBindingIfValid(Bindings.GBufferVelocity);
	}
	// Forward shading path. Simple forward shading does not use velocity.
	else if (IsUsingBasePassVelocity(ShaderPlatform))
	{
		const FGBufferBinding& GBufferVelocity = GBufferBindings[GBL_Default].GBufferVelocity;
		RenderTargetsInfo.RenderTargetFormats[RenderTargetCount] = GBufferVelocity.Format;
		RenderTargetsInfo.RenderTargetFlags[RenderTargetCount++] = GBufferVelocity.Flags;
	}

	if (bRequiresDepthAux)
	{
		RenderTargetsInfo.RenderTargetFormats[RenderTargetCount] = bPreciseDepthAux ? PF_R32_FLOAT : PF_R16F;
		RenderTargetsInfo.RenderTargetFlags[RenderTargetCount++] = TexCreate_RenderTargetable | TexCreate_ShaderResource | TexCreate_InputAttachmentRead;
	}
	
	if (bCustomResolveSubpass)
	{
		// resolve target as an additional color attachment
		// this is supposed to be be a swapchain pixel format, but atm there is no way to query it
		RenderTargetsInfo.RenderTargetFormats[RenderTargetCount] = IsAndroidPlatform(ShaderPlatform) ? PF_R8G8B8A8 : PF_B8G8R8A8;
		RenderTargetsInfo.RenderTargetFlags[RenderTargetCount++] = TexCreate_RenderTargetable | TexCreate_ShaderResource;
	}

	// Store final number of render targets
	RenderTargetsInfo.RenderTargetsEnabled = RenderTargetCount;

	// Precache TODO: other flags
	RenderTargetsInfo.MultiViewCount = 0;// RenderTargets.MultiViewCount;
	RenderTargetsInfo.bHasFragmentDensityAttachment = false; // RenderTargets.ShadingRateTexture != nullptr;

	return RenderTargetCount;
}
