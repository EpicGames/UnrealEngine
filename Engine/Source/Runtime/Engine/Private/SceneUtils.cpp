// Copyright Epic Games, Inc. All Rights Reserved.

#include "SceneUtils.h"
#include "HAL/IConsoleManager.h"
#include "RHI.h"
#include "Engine/Scene.h"
#include "RenderUtils.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "ComponentReregisterContext.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SceneUtils)

DEFINE_LOG_CATEGORY(LogSceneUtils);

static int32 GCustomDepthMode = 1;
static TAutoConsoleVariable<int32> CVarCustomDepth(
	TEXT("r.CustomDepth"),
	GCustomDepthMode,
	TEXT("0: feature is disabled\n")
	TEXT("1: feature is enabled, texture is created on demand\n")
	TEXT("2: feature is enabled, texture is not released until required (should be the project setting if the feature should not stall)\n")
	TEXT("3: feature is enabled, stencil writes are enabled, texture is not released until required (should be the project setting if the feature should not stall)"),
	ECVF_RenderThreadSafe);

void OnCustomDepthChanged(IConsoleVariable* Var)
{
	// Easiest way to update all static scene proxies is to recreate them
	FlushRenderingCommands();
	FGlobalComponentReregisterContext ReregisterContext;
	GCustomDepthMode = CVarCustomDepth.GetValueOnAnyThread();
}

void InitCustomDepth()
{
	CVarCustomDepth.AsVariable()->SetOnChangedCallback(FConsoleVariableDelegate::CreateStatic(&OnCustomDepthChanged));
}

ECustomDepthMode GetCustomDepthMode()
{
	switch (GCustomDepthMode)
	{
	case 1: // Fallthrough.
	case 2: return ECustomDepthMode::Enabled;
	case 3: return ECustomDepthMode::EnabledWithStencil;
	}
	return ECustomDepthMode::Disabled;
}

bool IsMobilePropagateAlphaEnabled(EShaderPlatform Platform)
{
	return IsMobilePlatform(Platform) && (FPlatformMisc::GetMobilePropagateAlphaSetting() > 0);
}

ENGINE_API bool IsMobileTonemapSubpassEnabled(EShaderPlatform Platform, bool bMultiViewRendering)
{
	static auto* MobileTonemapSubpassPathCvar = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.Mobile.TonemapSubpass"));
	return ((MobileTonemapSubpassPathCvar && (MobileTonemapSubpassPathCvar->GetValueOnAnyThread() == 1)) || bMultiViewRendering) && IsMobileHDR() && !IsMobileDeferredShadingEnabled(Platform);
}

ENGINE_API bool IsMobileTonemapSubpassEnabledInline(EShaderPlatform Platform, bool bMultiViewRendering, uint32 NumMSAASamples)
{
	// As of UE 5.4 only vulkan supports inline (single pass) tonemap
	return IsMobileTonemapSubpassEnabled(Platform, bMultiViewRendering) && IsVulkanPlatform(Platform) && (GRHISupportsMSAAShaderResolve || NumMSAASamples <= 1u);
}

ENGINE_API bool IsMobileColorsRGB()
{
	static auto* MobileUseHWsRGBEncodingCVAR = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.Mobile.UseHWsRGBEncoding"));
	const bool bMobileUseHWsRGBEncoding = (MobileUseHWsRGBEncodingCVAR && MobileUseHWsRGBEncodingCVAR->GetValueOnAnyThread() == 1);

	return !IsMobileHDR() && bMobileUseHWsRGBEncoding;
}

ENGINE_API EAntiAliasingMethod GetDefaultAntiAliasingMethod(const FStaticFeatureLevel InFeatureLevel)
{
	EAntiAliasingMethod AntiAliasingMethod = EAntiAliasingMethod::AAM_None;

	if (InFeatureLevel == ERHIFeatureLevel::ES3_1)
	{
		static auto* MobileAntiAliasingCvar = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.Mobile.AntiAliasing"));
		AntiAliasingMethod = EAntiAliasingMethod(FMath::Clamp<int32>(MobileAntiAliasingCvar->GetValueOnAnyThread(), 0, AAM_MAX));

		// Disable antialiasing in GammaLDR mode to avoid jittering.
		if (!IsMobileHDR() && AntiAliasingMethod != EAntiAliasingMethod::AAM_MSAA)
		{
			AntiAliasingMethod = EAntiAliasingMethod::AAM_None;
		}
	}
	else
	{
		static auto* DefaultAntiAliasingCvar = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.AntiAliasingMethod"));
		AntiAliasingMethod = EAntiAliasingMethod(FMath::Clamp<int32>(DefaultAntiAliasingCvar->GetValueOnAnyThread(), 0, AAM_MAX));
	}

	if (AntiAliasingMethod == AAM_MAX)
	{
		ensureMsgf(false, TEXT("Unknown anti-aliasing method."));
		AntiAliasingMethod = AAM_None;
	}

	static auto* MSAACountCVar = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.MSAACount"));

	EShaderPlatform ShaderPlatform = GetFeatureLevelShaderPlatform(InFeatureLevel);

	if (AntiAliasingMethod == EAntiAliasingMethod::AAM_None)
	{
		// NOP
	}
	else if (AntiAliasingMethod == EAntiAliasingMethod::AAM_FXAA)
	{
		// NOP
	}
	else if (AntiAliasingMethod == EAntiAliasingMethod::AAM_TemporalAA)
	{
		// NOP
	}
	else if (AntiAliasingMethod == EAntiAliasingMethod::AAM_MSAA)
	{
		if (MSAACountCVar->GetValueOnAnyThread() <= 0)
		{
			AntiAliasingMethod = EAntiAliasingMethod::AAM_None;
		}
		else if ((InFeatureLevel >= ERHIFeatureLevel::SM5 && !IsForwardShadingEnabled(ShaderPlatform))
			|| (IsMobilePlatform(ShaderPlatform) && IsMobileDeferredShadingEnabled(ShaderPlatform)))
		{
			// Disable antialiasing for deferred renderer
			AntiAliasingMethod = EAntiAliasingMethod::AAM_None;
		}
	}
	else if (AntiAliasingMethod == EAntiAliasingMethod::AAM_TSR)
	{
		if (!SupportsTSR(ShaderPlatform))
		{
			// Fallback to UE4's TAA if TSR isn't supported on that platform
			AntiAliasingMethod = AAM_TemporalAA;
		}
	}
	else if (AntiAliasingMethod == EAntiAliasingMethod::AAM_SMAA)
	{
		// NOP
	}
	else
	{
		unimplemented();
	}

	return AntiAliasingMethod;
}

ENGINE_API const TCHAR* GetShortAntiAliasingName(EAntiAliasingMethod AntiAliasingMethod)
{
	if (AntiAliasingMethod == AAM_None)
	{
		return TEXT("None");
	}
	else if (AntiAliasingMethod == AAM_FXAA)
	{
		return TEXT("FXAA");
	}
	else if (AntiAliasingMethod == AAM_TemporalAA)
	{
		return TEXT("TAA");
	}
	else if (AntiAliasingMethod == AAM_MSAA)
	{
		return TEXT("MSAA");
	}
	else if (AntiAliasingMethod == AAM_TSR)
	{
		return TEXT("TSR");
	}
	else if (AntiAliasingMethod == AAM_SMAA)
	{
		return TEXT("SMAA");
	}
	else
	{
		ensure(0);
	}

	return TEXT("UnknownAA");
}

ENGINE_API uint32 GetDefaultMSAACount(const FStaticFeatureLevel InFeatureLevel, uint32 PlatformMaxSampleCount/* = 8*/)
{
	uint32 NumSamples = 1;

	EAntiAliasingMethod AntiAliasingMethod = GetDefaultAntiAliasingMethod(InFeatureLevel);

	if (AntiAliasingMethod == EAntiAliasingMethod::AAM_MSAA)
	{
		static auto* MSAACountCVar = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.MSAACount"));

		EShaderPlatform ShaderPlatform = GetFeatureLevelShaderPlatform(InFeatureLevel);

		if (IsForwardShadingEnabled(ShaderPlatform) || (IsMobilePlatform(ShaderPlatform) && !IsMobileDeferredShadingEnabled(ShaderPlatform)))
		{
			NumSamples = FMath::Max(1, MSAACountCVar->GetValueOnAnyThread());


			NumSamples = FMath::Min(NumSamples, PlatformMaxSampleCount);

			if (NumSamples != 1 && NumSamples != 2 && NumSamples != 4 && NumSamples != 8)
			{
				UE_LOGF(LogSceneUtils, Warning, "Requested %d samples for MSAA, but this is not supported; falling back to 1 sample", NumSamples);
				NumSamples = 1;
			}
		}

		if (NumSamples > 1)
		{
			bool bRendererSupportMSAA = false;

			FString FailedReason;

			bool bRHISupportsMSAA = RHISupportsMSAA(ShaderPlatform);

			{
				bRendererSupportMSAA = bRHISupportsMSAA;

				if (!bRendererSupportMSAA)
				{
					FailedReason = FString::Printf(TEXT("RHISupportsMSAA %d"), bRHISupportsMSAA ? 1 : 0);
				}
			}

			if (!bRendererSupportMSAA)
			{
				static bool bWarned = false;

				if (!bWarned)
				{
					bWarned = true;
					UE_LOGF(LogSceneUtils, Log, "Requested %d samples for MSAA, but the platform doesn't support MSAA, failed reason : %ls", NumSamples, *FailedReason);
				}

				NumSamples = 1;
			}
		}
	}

	return NumSamples;
}
