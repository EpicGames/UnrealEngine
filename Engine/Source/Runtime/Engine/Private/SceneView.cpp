// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	SceneView.cpp: SceneView implementation.
=============================================================================*/

#include "SceneView.h"
#include "Engine/World.h"
#include "Materials/Material.h"
#include "Math/InverseRotationMatrix.h"
#include "Misc/Paths.h"
#include "Engine/Engine.h"
#include "SceneInterface.h"
#include "SceneManagement.h"
#include "ShaderCore.h"
#include "EngineModule.h"
#include "BufferVisualizationData.h"
#include "Engine/TextureCube.h"
#include "Engine/RendererSettings.h"
#include "Materials/MaterialInstanceDynamic.h"
#include "Materials/MaterialRenderProxy.h"
#include "HighResScreenshot.h"
#include "Slate/SceneViewport.h"
#include "RenderUtils.h"
#include "StereoRenderUtils.h"
#include "SceneRelativeViewMatrices.h"
#include "Camera/CameraComponent.h"
#include "Camera/CameraTypes.h"
#include "UObject/Interface.h"
#include "TextureResource.h"
#include "HDRHelper.h"
#include "VT/VirtualTextureScalability.h"
#include "SceneViewStateSystemMemory.h"
#include "RHIGPUReadback.h"

DEFINE_LOG_CATEGORY(LogBufferVisualization);
DEFINE_LOG_CATEGORY(LogNaniteVisualization);
DEFINE_LOG_CATEGORY(LogLumenVisualization);
DEFINE_LOG_CATEGORY(LogVirtualShadowMapVisualization);
DEFINE_LOG_CATEGORY(LogMultiView);

DECLARE_CYCLE_STAT(TEXT("StartFinalPostprocessSettings"), STAT_StartFinalPostprocessSettings, STATGROUP_Engine);
DECLARE_CYCLE_STAT(TEXT("OverridePostProcessSettings"), STAT_OverridePostProcessSettings, STATGROUP_Engine);

IMPLEMENT_STATIC_UNIFORM_BUFFER_SLOT(View);
IMPLEMENT_STATIC_UNIFORM_BUFFER_SLOT(InstancedView);

IMPLEMENT_GLOBAL_SHADER_PARAMETER_STRUCT(FPrimitiveUniformShaderParameters, "Primitive");
IMPLEMENT_STATIC_AND_SHADER_UNIFORM_BUFFER_STRUCT(FViewUniformShaderParameters, "View", View);
IMPLEMENT_STATIC_AND_SHADER_UNIFORM_BUFFER_STRUCT(FInstancedViewUniformShaderParameters, "InstancedView", InstancedView);
IMPLEMENT_GLOBAL_SHADER_PARAMETER_STRUCT(FMobileDirectionalLightShaderParameters, "MobileDirectionalLight");

static TAutoConsoleVariable<float> CVarSceneDepthFudgeFactor(
	TEXT("r.SceneDepthFudgeFactor"),
	1e-13f,
	TEXT("Subtract a tiny number to avoid divide by 0 errors in the shader when a very far distance is decided from the depth buffer."),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<float> CVarSSRMaxRoughness(
	TEXT("r.SSR.MaxRoughness"),
	-1.0f,
	TEXT("Allows to override the post process setting ScreenSpaceReflectionMaxRoughness.\n")
	TEXT("It defines until what roughness we fade the screen space reflections, 0.8 works well, smaller can run faster.\n")
	TEXT("(Useful for testing, no scalability or project setting)\n")
	TEXT(" 0..1: use specified max roughness (overrride PostprocessVolume setting)\n")
	TEXT(" -1: no override (default)"),
	ECVF_Scalability | ECVF_RenderThreadSafe);

static TAutoConsoleVariable<float> CVarGlobalMinRoughnessOverride(
	TEXT("r.MinRoughnessOverride"),
	0.0f,
	TEXT("WARNING: This is an experimental feature that may change at any time.\n")
	TEXT("Sets a global limit for roughness when used in the direct lighting calculations.\n")
	TEXT("This can be used to limit the amount of fireflies caused by low roughness, in particular when AA is not in use.\n")
	TEXT(" 0.0: no change (default)"),
	ECVF_Scalability | ECVF_RenderThreadSafe);

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)

static TAutoConsoleVariable<int32> CVarFreezeMouseCursor(
	TEXT("r.FreezeMouseCursor"),
	0,
	TEXT("Free the mouse cursor position, for passes which use it to display debug information.\n")
	TEXT("0: default\n")
	TEXT("1: freeze mouse cursor position at current location"),
	ECVF_Cheat);

static TAutoConsoleVariable<int32> CVarMouseCursorX(
	TEXT("r.MouseCursor.X"),
	-1,
	TEXT("Override default debug visualization cursor position. -1 disables override."),
	ECVF_Cheat);

static TAutoConsoleVariable<int32> CVarMouseCursorY(
	TEXT("r.MouseCursor.Y"),
	-1,
	TEXT("Override default debug visualization cursor position. -1 disables override."),
	ECVF_Cheat);

static TAutoConsoleVariable<int32> CVarShadowFreezeCamera(
	TEXT("r.Shadow.FreezeCamera"),
	0,
	TEXT("Debug the shadow methods by allowing to observe the system from outside.\n")
	TEXT("0: default\n")
	TEXT("1: freeze camera at current location"),
	ECVF_Cheat);

static TAutoConsoleVariable<int32> CVarRenderTimeFrozen(
	TEXT("r.RenderTimeFrozen"),
	0,
	TEXT("Allows to freeze time based effects in order to provide more deterministic render profiling.\n")
	TEXT(" 0: off\n")
	TEXT(" 1: on (Note: this also disables occlusion queries)"),
	ECVF_Cheat);

static TAutoConsoleVariable<float> CVarDepthOfFieldDepthBlurAmount(
	TEXT("r.DepthOfField.DepthBlur.Amount"),
	1.0f,
	TEXT("This scale multiplier only affects the CircleDOF DepthBlur feature (value defines in how many km the radius goes to 50%).\n")
	TEXT(" x: Multiply the existing Depth Blur Amount with x\n")
	TEXT("-x: Override the existing Depth Blur Amount with x (in km)\n")
	TEXT(" 1: No adjustments (default)"),
	ECVF_RenderThreadSafe | ECVF_Cheat);

static TAutoConsoleVariable<float> CVarDepthOfFieldDepthBlurScale(
	TEXT("r.DepthOfField.DepthBlur.Scale"),
	1.0f,
	TEXT("This scale multiplier only affects the CircleDOF DepthBlur feature. This is applied after r.DepthOfField.DepthBlur.ResolutionScale.\n")
	TEXT(" 0: Disable Depth Blur\n")
	TEXT(" x: Multiply the existing Depth Blur Radius with x\n")
	TEXT("-x: Override the existing Depth Blur Radius with x\n")
	TEXT(" 1: No adjustments (default)"),
	ECVF_RenderThreadSafe | ECVF_Cheat);

static TAutoConsoleVariable<float> CVarDepthOfFieldDepthBlurResolutionScale(
	TEXT("r.DepthOfField.DepthBlur.ResolutionScale"),
	1.0f,
	TEXT("This scale multiplier only affects the CircleDOF DepthBlur feature. It's a temporary hack.\n")
	TEXT("It lineary scale the DepthBlur by the resolution increase over 1920 (in width), does only affect resolution larger than that.\n")
	TEXT("Actual math: float Factor = max(ViewWidth / 1920 - 1, 0); DepthBlurRadius *= 1 + Factor * (CVar - 1)\n")
	TEXT(" 1: No adjustments (default)\n")
	TEXT(" x: if the resolution is 1920 there is no change, if 2x larger than 1920 it scale the radius by x"),
	ECVF_RenderThreadSafe | ECVF_Cheat);
#endif

static TAutoConsoleVariable<float> CVarSSAOFadeRadiusScale(
	TEXT("r.AmbientOcclusion.FadeRadiusScale"),
	1.0f,
	TEXT("Allows to scale the ambient occlusion fade radius (SSAO).\n")
	TEXT(" 0.01:smallest .. 1.0:normal (default), <1:smaller, >1:larger"),
	ECVF_Cheat | ECVF_RenderThreadSafe);

static TAutoConsoleVariable<float> CVarExposureOffset(
	TEXT("r.ExposureOffset"),
	0.0f,
	TEXT("For adjusting the exposure on top of post process settings and eye adaptation. 0: default"),
	ECVF_Cheat);

// Engine default (project settings):

static TAutoConsoleVariable<int32> CVarDefaultBloom(
	TEXT("r.DefaultFeature.Bloom"),
	1,
	TEXT("Engine default (project setting) for Bloom is (postprocess volume/camera/game setting still can override)\n")
	TEXT(" 0: off, set BloomIntensity to 0\n")
	TEXT(" 1: on (default)"));

static TAutoConsoleVariable<int32> CVarDefaultAmbientOcclusion(
	TEXT("r.DefaultFeature.AmbientOcclusion"),
	1,
	TEXT("Engine default (project setting) for AmbientOcclusion is (postprocess volume/camera/game setting still can override)\n")
	TEXT(" 0: off, sets AmbientOcclusionIntensity to 0\n")
	TEXT(" 1: on (default)"));

static TAutoConsoleVariable<int32> CVarDefaultAmbientOcclusionStaticFraction(
	TEXT("r.DefaultFeature.AmbientOcclusionStaticFraction"),
	1,
	TEXT("Engine default (project setting) for AmbientOcclusion is (postprocess volume/camera/game setting still can override)\n")
	TEXT(" 0: off, sets AmbientOcclusionStaticFraction to 0\n")
	TEXT(" 1: on (default, costs extra pass, only useful if there is some baked lighting)"));

static TAutoConsoleVariable<int32> CVarDefaultAutoExposure(
	TEXT("r.DefaultFeature.AutoExposure"),
	1,
	TEXT("Engine default (project setting) for AutoExposure is (postprocess volume/camera/game setting still can override)\n")
	TEXT(" 0: off, sets AutoExposureMinBrightness and AutoExposureMaxBrightness to 1\n")
	TEXT(" 1: on (default)"));

static TAutoConsoleVariable<int32> CVarDefaultAutoExposureMethod(
	TEXT("r.DefaultFeature.AutoExposure.Method"),
	0,
	TEXT("Engine default (project setting) for AutoExposure Method (postprocess volume/camera/game setting still can override)\n")
	TEXT(" 0: Histogram based (requires compute shader, default)\n")
	TEXT(" 1: Basic AutoExposure"));

static TAutoConsoleVariable<float> CVarDefaultAutoExposureBias(
	TEXT("r.DefaultFeature.AutoExposure.Bias"),
	1.0f,
	TEXT("Engine default (project setting) for AutoExposure Exposure Bias (postprocess volume/camera/game setting still can override)\n"));

static TAutoConsoleVariable<int32> CVarDefaultAutoExposureExtendDefaultLuminanceRange(
	TEXT("r.DefaultFeature.AutoExposure.ExtendDefaultLuminanceRange"),
	0,
	TEXT("Whether the default values for AutoExposure should support an extended range of scene luminance.\n")
	TEXT("This also change the PostProcessSettings.Exposure.MinBrightness, MaxBrightness, HistogramLogMin and HisogramLogMax\n")
	TEXT("to be expressed in EV100 values instead of in Luminance and Log2 Luminance.\n")
	TEXT(" 0: Legacy range (UE4 default)\n")
	TEXT(" 1: Extended range (UE5 default)"));

static TAutoConsoleVariable<float> CVarDefaultLocalExposureHighlightContrast(
	TEXT("r.DefaultFeature.LocalExposure.HighlightContrastScale"),
	1.0f,
	TEXT("Engine default (project setting) for Local Exposure Highlight Contrast (postprocess volume/camera/game setting still can override)\n"));

static TAutoConsoleVariable<float> CVarDefaultLocalExposureShadowContrast(
	TEXT("r.DefaultFeature.LocalExposure.ShadowContrastScale"),
	1.0f,
	TEXT("Engine default (project setting) for Local Exposure Shadow Contrast (postprocess volume/camera/game setting still can override)\n"));

static TAutoConsoleVariable<int32> CVarDefaultMotionBlur(
	TEXT("r.DefaultFeature.MotionBlur"),
	1,
	TEXT("Engine default (project setting) for MotionBlur is (postprocess volume/camera/game setting still can override)\n")
	TEXT(" 0: off, sets MotionBlurAmount to 0\n")
	TEXT(" 1: on (default)"));

// off by default for better performance and less distractions
static TAutoConsoleVariable<int32> CVarDefaultLensFlare(
	TEXT("r.DefaultFeature.LensFlare"),
	0,
	TEXT("Engine default (project setting) for LensFlare is (postprocess volume/camera/game setting still can override)\n")
	TEXT(" 0: off, sets LensFlareIntensity to 0\n")
	TEXT(" 1: on (default)"));

// see EAntiAliasingMethod
static TAutoConsoleVariable<int32> CVarDefaultAntiAliasing(
	TEXT("r.AntiAliasingMethod"),
	4,
	TEXT("Engine default (project setting) for AntiAliasingMethod is (postprocess volume/camera/game setting still can override)\n")
	TEXT(" 0: off (no anti-aliasing)\n")
	TEXT(" 1: Fast Approximate Anti-Aliasing (FXAA)\n")
	TEXT(" 2: Temporal Anti-Aliasing (TAA)\n")
	TEXT(" 3: Multisample Anti-Aliasing (MSAA, Only available on the desktop forward renderer)\n")
	TEXT(" 4: Temporal Super-Resolution (TSR, Default)\n")
	TEXT(" 5: Subpixel Morphological Anti-Aliasing (SMAA)"),
	ECVF_RenderThreadSafe);

// see ELightUnits
static TAutoConsoleVariable<int32> CVarDefaultPointLightUnits(
	TEXT("r.DefaultFeature.LightUnits"),
	1,
	TEXT("Default units to use for point, spot and rect lights\n")
	TEXT(" 0: unitless \n")
	TEXT(" 1: candelas (default)\n")
	TEXT(" 2: lumens"));

static TAutoConsoleVariable<float> CVarMotionBlurScale(
	TEXT("r.MotionBlur.Scale"),
	1.0f,
	TEXT("Allows to scale the postprocess intensity/amount setting in the postprocess.\n")
	TEXT("1: don't do any scaling (default)"),
	ECVF_Scalability | ECVF_RenderThreadSafe);

static TAutoConsoleVariable<float> CVarMotionBlurAmount(
	TEXT("r.MotionBlur.Amount"),
	-1.0f,
	TEXT("Allows to override the postprocess setting (scale of motion blur)\n")
	TEXT("-1: override (default)"),
	ECVF_Scalability | ECVF_RenderThreadSafe);

static TAutoConsoleVariable<float> CVarMotionBlurMax(
	TEXT("r.MotionBlur.Max"),
	-1.0f,
	TEXT("Allows to override the postprocess setting (max length of motion blur, in percent of the screen width)\n")
	TEXT("-1: override (default)"),
	ECVF_Scalability | ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarMotionBlurTargetFPS(
	TEXT("r.MotionBlur.TargetFPS"),
	-1,
	TEXT("Allows to override the postprocess setting (target FPS for motion blur velocity length scaling).\n")
	TEXT("-1: override (default)")
	TEXT(" 0: target current frame rate with moving average\n")
	TEXT("[1,120]: target FPS for motion blur velocity scaling"),
	ECVF_Scalability | ECVF_RenderThreadSafe);

static TAutoConsoleVariable<float> CVarSceneColorFringeMax(
	TEXT("r.SceneColorFringe.Max"),
	-1.0f,
	TEXT("Allows to clamp the postprocess setting (in percent, Scene chromatic aberration / color fringe to simulate an artifact that happens in real-world lens, mostly visible in the image corners)\n")
	TEXT("-1: don't clamp (default)\n")
	TEXT("-2: to test extreme fringe"),
	ECVF_Scalability | ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarTonemapperQuality(
	TEXT("r.Tonemapper.Quality"),
	5,
	TEXT("Defines the Tonemapper Quality in the range 0..5\n")
	TEXT("Depending on the used settings we might pick a faster shader permutation\n")
	TEXT(" 0: basic tonemapper only, lowest quality\n")
	TEXT(" 2: + Vignette\n")
	TEXT(" 4: + Grain\n")
	TEXT(" 5: (default)"),
	ECVF_Scalability | ECVF_RenderThreadSafe);

// should be changed to BaseColor and Metallic, since some time now UE is not using DiffuseColor and SpecularColor any more
static TAutoConsoleVariable<float> CVarDiffuseColorMin(
	TEXT("r.DiffuseColor.Min"),
	0.0f,
	TEXT("Allows quick material test by remapping the diffuse color at 1 to a new value (0..1), Only for non shipping built!\n")
	TEXT("1: (default)"),
	ECVF_Cheat | ECVF_RenderThreadSafe
	);
static TAutoConsoleVariable<float> CVarDiffuseColorMax(
	TEXT("r.DiffuseColor.Max"),
	1.0f,
	TEXT("Allows quick material test by remapping the diffuse color at 1 to a new value (0..1), Only for non shipping built!\n")
	TEXT("1: (default)"),
	ECVF_Cheat | ECVF_RenderThreadSafe
	);
static TAutoConsoleVariable<float> CVarRoughnessMin(
	TEXT("r.Roughness.Min"),
	0.0f,
	TEXT("Allows quick material test by remapping the roughness at 0 to a new value (0..1), Only for non shipping built!\n")
	TEXT("0: (default)"),
	ECVF_Cheat | ECVF_RenderThreadSafe
	);
static TAutoConsoleVariable<float> CVarRoughnessMax(
	TEXT("r.Roughness.Max"),
	1.0f,
	TEXT("Allows quick material test by remapping the roughness at 1 to a new value (0..1), Only for non shipping built!\n")
	TEXT("1: (default)"),
	ECVF_Cheat | ECVF_RenderThreadSafe
	);

static TAutoConsoleVariable<int32> CVarEnableTemporalUpsample(
	TEXT("r.TemporalAA.Upsampling"),
	1,
	TEXT("Whether to do primary screen percentage with temporal AA or not.\n")
	TEXT(" 0: use spatial upscale pass independently of TAA;\n")
	TEXT(" 1: TemporalAA performs spatial and temporal upscale as screen percentage method (default)."),
	ECVF_Default);

static TAutoConsoleVariable<int32> CVarOrthoCalculateDepthThicknessScaling(
	TEXT("r.Ortho.CalculateDepthThicknessScaling"),
	1,
	TEXT("Whether to automatically derive the depth thickness test scale from the Near/FarPlane difference.\n")
	TEXT("0: Disabled (use scaling specified by r.Ortho.DepthThicknessScale)\n")
	TEXT("1: Enabled (default)"),
	ECVF_Scalability | ECVF_RenderThreadSafe);

float GOrthographicDepthThicknessScale = 0.001f;
static FAutoConsoleVariableRef CVarOrthographicDepthThicknessScale(
	TEXT("r.Ortho.DepthThicknessScale"),
	GOrthographicDepthThicknessScale,
	TEXT("Orthographic scene depth scales proportionally lower than perspective, typically on a scale of 1/100")
	TEXT("Use this value to tweak the scale of depth thickness testing values simultaneously across various screen trace passes"),
	ECVF_RenderThreadSafe);

float GDefaultUpdateOrthoNearPlane = 0.0f;
static FAutoConsoleVariableRef CVarDefaultUpdateOrthoNearPlane(
	TEXT("r.Ortho.DefaultUpdateNearClipPlane"),
	GDefaultUpdateOrthoNearPlane,
	TEXT("Ortho near clip plane value to correct to when using ortho near clip correction"),
	ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<bool> CVarAllowOrthoNearPlaneCorrection(
	TEXT("r.Ortho.AllowNearPlaneCorrection"),
	true,
	TEXT("Orthographic near planes can be behind the camera position, which causes some issues with Unreal resolving lighting behind the camera position ")
	TEXT("This CVar enables the Orthographic cameras globally to automatically update the camera location to match the NearPlane location, ")
	TEXT("and force the pseudo camera position to be the replaced near plane location for the projection matrix calculation. ")
	TEXT("This means Unreal can resolve lighting behind the camera correctly."),
	ECVF_Default
);

static TAutoConsoleVariable<bool> CVarOrthoCameraHeightAsViewTarget(
	TEXT("r.Ortho.CameraHeightAsViewTarget"),
	true,
	TEXT("Sets whether to use the camera height as a pseudo camera to view target.\n")
	TEXT("Primarily helps with VSM clipmap selection and avoids overcorrecting NearPlanes.\n"),
	ECVF_Scalability | ECVF_RenderThreadSafe);

static int32 GHairStrandsComposeDOFDepth = 1;
static FAutoConsoleVariableRef CVarHairStrandsComposeDOFDepth(
	TEXT("r.HairStrands.DOFDepth"), 
	GHairStrandsComposeDOFDepth, 
	TEXT("Compose hair with DOF by lerping hair depth based on its opacity."),
	ECVF_RenderThreadSafe);

bool GetHairStrandsDepthOfFieldUseHairDepth()
{
	return GHairStrandsComposeDOFDepth > 0 ? 1 : 0;
}

#if !(UE_BUILD_SHIPPING)

static TAutoConsoleVariable<float> CVarOverrideTimeMaterialExpressions(
	TEXT("r.Test.OverrideTimeMaterialExpressions"), -1.0f,
	TEXT("Value to freeze time material expressions with."),
	ECVF_RenderThreadSafe);

#endif

/** Global primitive uniform buffer resource containing identity transformations. */
ENGINE_API TGlobalResource<FIdentityPrimitiveUniformBuffer> GIdentityPrimitiveUniformBuffer;

FSceneViewStateReference::~FSceneViewStateReference()
{
	checkf(ShareOriginRefCount == 0, TEXT("FSceneViewStateReference:  ShareOrigin view states must be destroyed before their target."));

	if (ShareOriginTarget)
	{
		ShareOriginTarget->ShareOriginRefCount--;
		ShareOriginTarget = nullptr;
	}

	Destroy();
}

void FSceneViewStateReference::AllocateInternal(ERHIFeatureLevel::Type FeatureLevel)
{
	FSceneViewStateInterface* ShareOriginInterface = nullptr;
	if (ShareOriginTarget)
	{
		check(ShareOriginTarget->Reference);
		ShareOriginInterface = ShareOriginTarget->Reference;
	}

	Reference = GetRendererModule().AllocateViewState(FeatureLevel, ShareOriginInterface);
}

void FSceneViewStateReference::Allocate(ERHIFeatureLevel::Type FeatureLevel)
{
	check(!Reference);

	AllocateInternal(FeatureLevel);

	GlobalListLink = TLinkedList<FSceneViewStateReference*>(this);
	GlobalListLink.LinkHead(GetSceneViewStateList());
}

ENGINE_API void FSceneViewStateReference::ShareOrigin(FSceneViewStateReference* Target)
{
	checkf(ShareOriginTarget == nullptr, TEXT("FSceneViewStateReference:  Cannot call ShareOrigin twice."));
	checkf(ShareOriginRefCount == 0 && Target->ShareOriginTarget == nullptr, TEXT("FSceneViewStateReference:  ShareOrigin references cannot nest."));
	checkf(Reference == nullptr, TEXT("FSceneViewStateReference:  Must call ShareOrigin before calling Allocate."));

	ShareOriginTarget = Target;
	Target->ShareOriginRefCount++;
}

void FSceneViewStateReference::Destroy()
{
	GlobalListLink.Unlink();

	if (Reference)
	{
		Reference->Destroy();
		Reference = NULL;
	}
}

TLinkedList<FSceneViewStateReference*>*& FSceneViewStateReference::GetSceneViewStateList()
{
	static TLinkedList<FSceneViewStateReference*>* List = NULL;
	return List;
}

FString LexToString(EMaterialQualityLevel::Type QualityLevel)
{
	switch (QualityLevel)
	{
		case EMaterialQualityLevel::Low:
			return TEXT("Low");
		case EMaterialQualityLevel::High:
			return TEXT("High");
		case EMaterialQualityLevel::Medium:
			return TEXT("Medium");
		case EMaterialQualityLevel::Epic:
			return TEXT("Epic");
		case EMaterialQualityLevel::Num:
			return TEXT("Default");
		default:
			break;
	}
	return TEXT("UnknownMaterialQualityLevel");
}

/**
 * Utility function to create the inverse depth projection transform to be used
 * by the shader system.
 * @param ProjMatrix - used to extract the scene depth ratios
 * @param InvertZ - projection calc is affected by inverted device Z
 * @return vector containing the ratios needed to convert from device Z to world Z
 */
FVector4f CreateInvDeviceZToWorldZTransform(const FMatrix& ProjMatrix)
{
	// The perspective depth projection comes from the the following projection matrix:
	//
	// | 1  0  0  0 |
	// | 0  1  0  0 |
	// | 0  0  A  1 |
	// | 0  0  B  0 |
	//
	// Z' = (Z * A + B) / Z
	// Z' = A + B / Z
	//
	// So to get Z from Z' is just:
	// Z = B / (Z' - A)
	//
	// Note a reversed Z projection matrix will have A=0.
	//
	// Done in shader as:
	// Z = 1 / (Z' * C1 - C2)   --- Where C1 = 1/B, C2 = A/B
	//

	float DepthMul = (float)ProjMatrix.M[2][2];
	float DepthAdd = (float)ProjMatrix.M[3][2];

	if (DepthAdd == 0.f)
	{
		// Avoid dividing by 0 in this case
		DepthAdd = 0.00000001f;
	}

	// perspective
	// SceneDepth = 1.0f / (DeviceZ / ProjMatrix.M[3][2] - ProjMatrix.M[2][2] / ProjMatrix.M[3][2])

	// ortho
	// SceneDepth = DeviceZ / ProjMatrix.M[2][2] - ProjMatrix.M[3][2] / ProjMatrix.M[2][2];

	// combined equation in shader to handle either
	// SceneDepth = DeviceZ * View.InvDeviceZToWorldZTransform[0] + View.InvDeviceZToWorldZTransform[1] + 1.0f / (DeviceZ * View.InvDeviceZToWorldZTransform[2] - View.InvDeviceZToWorldZTransform[3]);

	// therefore perspective needs
	// View.InvDeviceZToWorldZTransform[0] = 0.0f
	// View.InvDeviceZToWorldZTransform[1] = 0.0f
	// View.InvDeviceZToWorldZTransform[2] = 1.0f / ProjMatrix.M[3][2]
	// View.InvDeviceZToWorldZTransform[3] = ProjMatrix.M[2][2] / ProjMatrix.M[3][2]

	// and ortho needs
	// View.InvDeviceZToWorldZTransform[0] = 1.0f / ProjMatrix.M[2][2]
	// View.InvDeviceZToWorldZTransform[1] = -ProjMatrix.M[3][2] / ProjMatrix.M[2][2] + 1.0f
	// View.InvDeviceZToWorldZTransform[2] = 0.0f
	// View.InvDeviceZToWorldZTransform[3] = 1.0f

	bool bIsPerspectiveProjection = ProjMatrix.M[3][3] < 1.0f;

	if (bIsPerspectiveProjection)
	{
		float SubtractValue = DepthMul / DepthAdd;

		// Subtract a tiny number to avoid divide by 0 errors in the shader when a very far distance is decided from the depth buffer.
		// This fixes fog not being applied to the black background in the editor.
		SubtractValue -= CVarSceneDepthFudgeFactor.GetValueOnAnyThread();

		return FVector4f(
			0.0f,
			0.0f,
			1.0f / DepthAdd,
			SubtractValue
			);
	}
	else
	{
		return FVector4f(
			(float)(1.0f / ProjMatrix.M[2][2]),
			(float)(-ProjMatrix.M[3][2] / ProjMatrix.M[2][2] + 1.0f),
			0.0f,
			1.0f
			);
	}
}

bool FSceneViewProjectionData::UpdateOrthoPlanes(FSceneViewProjectionData* InOutProjectionData, float& NearPlane, float& FarPlane, float HalfOrthoWidth, bool bUseCameraHeightAsViewTarget)
{
	if (!InOutProjectionData || !CVarAllowOrthoNearPlaneCorrection.GetValueOnAnyThread())
	{
		return false;
	}
	
	//Get the ViewForward vector from the RotationMatrix + ensure it is normalized.
	FVector ViewForward = InOutProjectionData->ViewRotationMatrix.GetColumn(2);
	ViewForward.Normalize();

	float PlaneDifference = FarPlane - NearPlane;
	if(InOutProjectionData->CameraToViewTarget.Length() > 0)
	{
		//Store the ViewTargetLocation to correct it for the repositioned ViewOrigin.
		FVector ViewTargetLocation = InOutProjectionData->ViewOrigin + InOutProjectionData->CameraToViewTarget;
		//OrthoNearClipPlane is negative at this point + we are moving the theoretical camera position backwards.
		InOutProjectionData->ViewOrigin += InOutProjectionData->CameraToViewTarget +(ViewForward * NearPlane);
		//Save out the new CameraToViewTarget vector.
		InOutProjectionData->CameraToViewTarget = ViewTargetLocation - InOutProjectionData->ViewOrigin;
	}
	else
	{	
		/**
		* Use the height of the camera as a sort of pseudo CameraToViewTarget to remove camera position as much as possible so we minimise the near clip plane distance.
		* Depends on view direction being top down as when we get to a side view, the scale grows much larger and it becomes redundant.
		*/
		float CameraHeightAdjustment = 0.0f;
		if (CVarOrthoCameraHeightAsViewTarget.GetValueOnAnyThread() && bUseCameraHeightAsViewTarget)
		{
			CameraHeightAdjustment = FMath::Abs(FMath::Min(InOutProjectionData->ViewOrigin.Z, HalfOrthoWidth)) * FMath::Abs(ViewForward.Z);
		}
		InOutProjectionData->ViewOrigin += ViewForward * (CameraHeightAdjustment + NearPlane);
	}
	NearPlane = GDefaultUpdateOrthoNearPlane;
	FarPlane = NearPlane + PlaneDifference;

	return true;
}

bool FSceneViewProjectionData::UpdateOrthoPlanes(FMinimalViewInfo& MinimalViewInfo)
{
	if(MinimalViewInfo.bUpdateOrthoPlanes)
	{
		return UpdateOrthoPlanes(MinimalViewInfo.OrthoNearClipPlane, MinimalViewInfo.OrthoFarClipPlane, MinimalViewInfo.OrthoWidth/2.0f, MinimalViewInfo.bUseCameraHeightAsViewTarget);
	}
	return false;
}

bool FSceneViewProjectionData::UpdateOrthoPlanes(bool bUseCameraHeightAsViewTarget)
{
	/**
	* This function takes the existing projection matrix and moves the nearplane + view origin.
	* Separating this step from the near plane calculation logic itself helps avoid applying the NearPlane correctiontwice /makes it easier to read.
	*/
	if(IsPerspectiveProjection() 
		|| !CVarAllowOrthoNearPlaneCorrection.GetValueOnAnyThread()
		|| ProjectionMatrix.M[2][2] == 0.0 
		|| ProjectionMatrix.M[2][2] == ProjectionMatrix.M[2][3])
	{ 
		return false;	
	}

	//Get existing Near and Far plane + their difference
	float NearPlane = static_cast<float>(ProjectionMatrix.M[3][3] - ProjectionMatrix.M[3][2]) / (ProjectionMatrix.M[2][2] - ProjectionMatrix.M[2][3]);
	float FarPlane = NearPlane - 1.0f/static_cast<float>(ProjectionMatrix.M[2][2]);
	float PlaneDifference = FarPlane - NearPlane;
	if (FarPlane - NearPlane == 0.0f)
	{
		return false;
	}

	float HalfInvOrthoWidth = static_cast<float>(ProjectionMatrix.M[0][0]);
	if (HalfInvOrthoWidth == 0.0f)
	{
		return false;
	}
	UpdateOrthoPlanes(NearPlane, FarPlane, 1.0f / HalfInvOrthoWidth, bUseCameraHeightAsViewTarget);

	const float ZScale = 1.0f / (FarPlane - NearPlane);
	const float ZOffset = -NearPlane;

	//Only the Near/Far plane elements need correcting, OrthoWidth/Height remains the same
	ProjectionMatrix.M[2][2] = -ZScale;
	ProjectionMatrix.M[3][2] = 1.0f - (ZOffset * ZScale);

	return true;
}

void FViewMatrices::Init(const FMinimalInitializer& Initializer)
{
	FMatrix ViewRotationMatrix = Initializer.ViewRotationMatrix;
	const FVector ViewRotationScaling = ViewRotationMatrix.ExtractScaling();
	ensureMsgf(FVector::Distance(ViewRotationScaling, FVector::OneVector) < UE_KINDA_SMALL_NUMBER, TEXT("ViewRotation matrix accumulated scaling (%f, %f, %f)"), ViewRotationScaling.X, ViewRotationScaling.Y, ViewRotationScaling.Z);

	// Adjust the projection matrix for the current RHI.
	ViewToClip = AdjustProjectionMatrixForRHI(Initializer.ProjectionMatrix);
	ClipToView = InvertProjectionMatrix(ViewToClip);
	ScreenToClipMatrix = ScreenToClipProjectionMatrix();

	FVector LocalViewOrigin = Initializer.ViewOrigin;
	if (!ViewRotationMatrix.GetOrigin().IsNearlyZero(0.0f))
	{
		LocalViewOrigin += ViewRotationMatrix.InverseTransformPosition(FVector::ZeroVector);
		ViewRotationMatrix = ViewRotationMatrix.RemoveTranslation();
	}

	WorldToView = FTranslationMatrix(-LocalViewOrigin) * ViewRotationMatrix;
	HMDViewMatrixNoRoll = Initializer.ViewRotationMatrix;

	// Compute the view projection matrix and its inverse.
	WorldToClip = GetWorldToView() * GetViewToClip();

	// For precision reasons the view matrix inverse is calculated independently.
	ViewToWorld = ViewRotationMatrix.GetTransposed() * FTranslationMatrix(LocalViewOrigin);
	ClipToWorld = ClipToView * ViewToWorld;

	// Translate world-space so its origin is at ViewOrigin for improved precision.
	ViewOrigin = LocalViewOrigin;
	PreViewTranslation = -LocalViewOrigin;
	CameraToViewTarget = Initializer.CameraToViewTarget;

	FMatrix LocalTranslatedViewMatrix = ViewRotationMatrix;
	FMatrix LocalInvTranslatedViewMatrix = LocalTranslatedViewMatrix.GetTransposed();

	// Compute a transform from view origin centered world-space to clip space.
	TranslatedViewMatrix = LocalTranslatedViewMatrix;
	InvTranslatedViewMatrix = LocalInvTranslatedViewMatrix;

	OverriddenTranslatedViewMatrix = LocalTranslatedViewMatrix;
	OverriddenInvTranslatedViewMatrix = LocalInvTranslatedViewMatrix;

	TranslatedWorldToClip = LocalTranslatedViewMatrix * ViewToClip;
	ClipToTranslatedWorld = ClipToView * LocalInvTranslatedViewMatrix;

	// Compute screen scale factors.
	// Stereo renders at half horizontal resolution, but compute shadow resolution based on full resolution.
	const bool bStereo = IStereoRendering::IsStereoEyePass(Initializer.StereoPass);
	const float ScreenXScale = bStereo ? 2.0f : 1.0f;
	if (IsPerspectiveProjection())
	{
		ProjectionScale.X = ScreenXScale * FMath::Abs(ViewToClip.M[0][0]);
		ProjectionScale.Y = FMath::Abs(ViewToClip.M[1][1]);
		PerProjectionDepthThicknessScale = 1.0f;
	}
	else
	{
		//No FOV for ortho so do not scale
		ProjectionScale = FVector2D(ScreenXScale, 1.0f);

		if (CVarOrthoCalculateDepthThicknessScaling.GetValueOnAnyThread())
		{
			int8 Exponent = -FMath::Clamp(FMath::LogX(10, FMath::Abs(ClipToView.M[2][2])), 0, 9);
			PerProjectionDepthThicknessScale = FMath::Pow(10, (float)Exponent);
		}
		else
		{
			PerProjectionDepthThicknessScale = GOrthographicDepthThicknessScale;
		}
	}
	ScreenScale = FMath::Max(
		Initializer.ConstrainedViewRect.Size().X * 0.5f * ProjectionScale.X,
		Initializer.ConstrainedViewRect.Size().Y * 0.5f * ProjectionScale.Y
	);

	FirstPersonParams = Initializer.FirstPersonParams;

	if (FirstPersonParams.bUseParameters)
	{
		// The projection matrix uses 1/tan(HalfFOV) on the X and Y components in view space, so in order to cancel out the existing FOV and achieve a different first person FOV, we do
		//		(1 / tan(HalfFirstPersonFOV)) / (1 / tan(HalfFOV)) 
		// which is equivalent to
		//		tan(HalfFOV) / tan(HalfFirstPersonFOV)
		// This correction factor should be independent of the actual aspect ratio used on the original projection matrix. As the FOV correction needs to be done in view space, we can conveniently
		// also use this opportunity to scale down the first person geometry towards the camera. The geometry takes up the same solid angle from the perspective of the camera and therefore appears the
		// same size while taking up a smaller range in the depth buffer, reducing the likelihood of intersections with the scene.
		const float FOVCorrectionFactor = FirstPersonParams.FOVCorrectionFactor;
		const FVector ScaleVector = FVector(FOVCorrectionFactor, FOVCorrectionFactor, 1.0f) * FirstPersonParams.Scale;
		WorldToFirstPerson = TranslatedViewMatrix * FScaleMatrix(ScaleVector) * InvTranslatedViewMatrix;
		FirstPersonToWorld = TranslatedViewMatrix * FScaleMatrix(ScaleVector.Reciprocal()) * InvTranslatedViewMatrix;
	}
}

FViewMatrices::FViewMatrices(const FSceneViewInitOptions& InitOptions) : FViewMatrices()
{
	FMinimalInitializer Initializer;

	Initializer.ViewRotationMatrix   = InitOptions.ViewRotationMatrix;
	Initializer.ProjectionMatrix     = InitOptions.ProjectionMatrix;
	Initializer.ViewOrigin           = InitOptions.ViewOrigin;
	Initializer.CameraToViewTarget	 = InitOptions.CameraToViewTarget;
	Initializer.ConstrainedViewRect  = InitOptions.GetConstrainedViewRect();
	Initializer.StereoPass           = InitOptions.StereoPass;
	Initializer.FirstPersonParams    = InitOptions.FirstPersonParams;

	Init(Initializer);
}

FViewMatrices::FViewMatrices(const FMinimalInitializer& Initializer) : FViewMatrices()
{
	Init(Initializer);
}

FSceneViewOwner::FSceneViewOwner(const AActor* InActor)
	: ActorUniqueId(InActor ? InActor->GetUniqueID() : 0)
#if !WITH_STATE_STREAM
	, Actor(InActor)
#endif
{
}

#if !WITH_STATE_STREAM
void FSceneViewOwner::operator=(const AActor* InActor)
{
	Actor = InActor;
	ActorUniqueId = InActor ? InActor->GetUniqueID() : 0;
}
#endif

void FSceneView::SetupViewFrustum()
{
	if (SceneViewInitOptions.OverrideFarClippingPlaneDistance > 0.0f)
	{
		const FPlane FarPlane(ViewMatrices.GetViewOrigin() + GetViewDirection() * SceneViewInitOptions.OverrideFarClippingPlaneDistance, GetViewDirection());
		// Derive the view frustum from the view projection matrix, overriding the far plane
		GetViewFrustumBounds(ViewFrustum, ViewMatrices.GetWorldToClip(), FarPlane, true, false);
	}
	else
	{
		// Derive the view frustum from the view projection matrix.
		GetViewFrustumBounds(ViewFrustum, ViewMatrices.GetWorldToClip(), false);
	}

	// Use a unified frustum for culling in single pass stereo (i.e. when we're drawing both eyes in one pass rather than
	// rendering them separately as separate views).
	if (bIsSinglePassStereo && IStereoRendering::IsStereoEyeView(*this) && GEngine->StereoRenderingDevice.IsValid())
	{
		StereoCullingFrustum = MakeShared<FConvexVolume>();

		FVector MonoLocation = ViewLocation;
		FRotator MonoRotation = ViewRotation;
		GEngine->StereoRenderingDevice->CalculateStereoViewOffset(eSSE_MONOSCOPIC, MonoRotation, WorldToMetersScale, MonoLocation);
		const FMatrix ViewRotationMatrix = FInverseRotationMatrix(MonoRotation) * FMatrix(
			FPlane(0, 0, 1, 0),
			FPlane(1, 0, 0, 0),
			FPlane(0, 1, 0, 0),
			FPlane(0, 0, 0, 1));
		const FMatrix ViewMatrixForCulling = FTranslationMatrix(-MonoLocation) * ViewRotationMatrix;
		const FMatrix ViewProjForCulling = ViewMatrixForCulling * GEngine->StereoRenderingDevice->GetStereoProjectionMatrix(eSSE_MONOSCOPIC);

		if (SceneViewInitOptions.OverrideFarClippingPlaneDistance > 0.0f)
		{
			const FPlane FarPlane(ViewMatrices.GetViewOrigin() + GetViewDirection() * SceneViewInitOptions.OverrideFarClippingPlaneDistance, GetViewDirection());
			// Derive the frustum from the view projection matrix, overriding the far plane
			GetViewFrustumBounds(*StereoCullingFrustum, ViewProjForCulling, FarPlane, true, false);
		}
		else
		{
			// Derive the frustum from the view projection matrix.
			GetViewFrustumBounds(*StereoCullingFrustum, ViewProjForCulling, false);
		}

		CullingOrigin = MonoLocation;
	}
	else
	{
		StereoCullingFrustum = nullptr;
		CullingOrigin = ViewMatrices.GetViewOrigin();
	}

	// Derive the view's near clipping distance and plane.
	bHasNearClippingPlane = ViewMatrices.GetWorldToClip().GetFrustumNearPlane(NearClippingPlane);
	NearClippingDistance = ViewMatrices.ComputeNearPlane();
}

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
namespace ShadowFreezeCamera
{
	static TOptional<FViewMatrices> Backup;

	void Init(const FViewMatrices& Matrices)
	{
		const bool bShadowFreezeCamera = CVarShadowFreezeCamera.GetValueOnAnyThread() != 0;

		if (bShadowFreezeCamera)
		{
			if (!Backup)
			{
				Backup.Emplace(Matrices);
			}
		}
		else
		{
			Backup.Reset();
		}
	}
}
#endif

FSceneView::FSceneView(const FSceneViewInitOptions& InitOptions)
	: Family(InitOptions.ViewFamily)
	, State(InitOptions.SceneViewStateInterface)
	, DynamicMeshElementsShadowCullFrustum(nullptr)
	, PreShadowTranslation(FVector::ZeroVector)
	, ViewActor(InitOptions.ViewActor)
	, PlayerIndex(InitOptions.PlayerIndex)
	, Drawer(InitOptions.ViewElementDrawer)
	, UnscaledViewRect(InitOptions.GetConstrainedViewRect())
	, UnconstrainedViewRect(InitOptions.GetViewRect())
	, MaxShadowCascades(10)
	, ViewMatrices(InitOptions)
	, ViewLocation(InitOptions.ViewLocation)
	, ViewRotation(InitOptions.ViewRotation)
	, BaseHmdOrientation(EForceInit::ForceInit)
	, BaseHmdLocation(ForceInitToZero)
	, WorldToMetersScale(InitOptions.WorldToMetersScale)
	, ProjectionMatrixUnadjustedForRHI(InitOptions.ProjectionMatrix)
	, BackgroundColor(InitOptions.BackgroundColor)
	, OverlayColor(InitOptions.OverlayColor)
	, ColorScale(InitOptions.ColorScale)
	, SkylightScale(InitOptions.SkylightScale)
	, StereoPass(InitOptions.StereoPass)
	, StereoViewIndex(InitOptions.StereoViewIndex)
	, PrimaryViewIndex(INDEX_NONE)
	, bAllowCrossGPUTransfer(true)
	, bOverrideGPUMask(false)
	, GPUMask(FRHIGPUMask::GPU0())
	, bRenderFirstInstanceOnly(false)
	, DiffuseOverrideParameter(FVector4(0,0,0,1))
	, SpecularOverrideParameter(FVector4(0,0,0,1))
	, NormalOverrideParameter(FVector4(0,0,0,1))
	, RoughnessOverrideParameter(FVector2f(0,1))
	, AmbientOcclusionOverrideParameter(0.0f, 1.0f)
	, MaterialTextureMipBias(0.f)
	, HiddenPrimitives(InitOptions.HiddenPrimitives)
	, ShowOnlyPrimitives(InitOptions.ShowOnlyPrimitives)
	, OriginOffsetThisFrame(InitOptions.OriginOffsetThisFrame)
	, LODDistanceFactor(InitOptions.LODDistanceFactor)
	, bCameraCut(InitOptions.bInCameraCut)
	, CursorPos(InitOptions.CursorPos)
	, bIsGameView(false)
	, bIsViewInfo(false)
	, bIsSceneCapture(InitOptions.bIsSceneCapture)
	, bIsSceneCaptureCube(InitOptions.bIsSceneCaptureCube)
	, bEyeAdaptationAllViewPixels(false)
	, bSceneCaptureUsesRayTracing(InitOptions.bSceneCaptureUsesRayTracing)
	, bIsReflectionCapture(InitOptions.bIsReflectionCapture)
	, bIsPlanarReflection(InitOptions.bIsPlanarReflection)
	, bIsVirtualTexture(false)
	, bIsOfflineRender(false)
	, bRenderSceneTwoSided(false)
	, bIsLocked(false)
	, bStaticSceneOnly(false)
	, bIsInstancedStereoEnabled(false)
	, bIsMultiViewportEnabled(false)
	, bIsMobileMultiViewEnabled(false)
	, bShouldBindInstancedViewUB(false)
	, bIsSinglePassStereo(false)
	, bForceCameraVisibilityReset(false)
	, bDisableDistanceBasedFadeTransitions(false)
	, bExcludeFromSceneTextureExtents(InitOptions.bExcludeFromSceneTextureExtents)
	, bForcePathTracerReset(false)
	, PathTracerSeedOffset(0)
	, UnderwaterDepth(-1.0f)
	, GlobalClippingPlane(FPlane(0, 0, 0, 0))
	, LensPrincipalPointOffsetScale(0.0f, 0.0f, 1.0f, 1.0f)
#if WITH_EDITOR
	, bAllowTranslucentPrimitivesInHitProxy( true )
	, bHasSelectedComponents( false )
#endif
	, AntiAliasingMethod(AAM_None)
	, PrimaryScreenPercentageMethod(EPrimaryScreenPercentageMethod::SpatialUpscale)
	, FeatureLevel(InitOptions.ViewFamily ? InitOptions.ViewFamily->GetFeatureLevel() : GMaxRHIFeatureLevel)
{
	SceneViewInitOptions = FSceneViewInitOptions(InitOptions);

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	ShadowFreezeCamera::Init(ViewMatrices);
#endif // !(UE_BUILD_SHIPPING || UE_BUILD_TEST)

	auto ShaderPlatform = GShaderPlatformForFeatureLevel[FeatureLevel];

	// Query instanced stereo and multi-view state
	{
		// The shader variants that are compiled have ISR or MMV _enabled_ in the shader, even if the current ViewFamily doesn't
		// require multiple views functionality.
		Aspects = UE::StereoRenderUtils::FStereoShaderAspects(ShaderPlatform);
		bShouldBindInstancedViewUB = Aspects.IsInstancedStereoEnabled() || Aspects.IsMobileMultiViewEnabled();

		if (Family && Family->bRequireMultiView && !Aspects.IsMobileMultiViewEnabled())
		{
			UE_LOGF(LogMultiView, Fatal, "Family requires Mobile Multi-View, but it is not supported by the RHI and no fallback is available.");
		}

		if (Aspects.IsSinglePassStereoEnabled())
		{
			checkf((Aspects.IsInstancedStereoEnabled() || Aspects.IsMobileMultiViewEnabled()) &&
				!(Aspects.IsInstancedStereoEnabled() && Aspects.IsMobileMultiViewEnabled()),
				TEXT("If single pass stereo is enabled, either instanced stereo or mobile multi-view needs to be enabled as well but not both."));

			// Planar reflections have bIsSceneCapture == true, but can have instanced stereo views
			const bool bIsSingleViewCapture = (bIsSceneCapture || bIsReflectionCapture) && !bIsPlanarReflection;
			bIsInstancedStereoEnabled = !bIsSingleViewCapture && Aspects.IsInstancedStereoEnabled();
			
			// Deprecated
			PRAGMA_DISABLE_DEPRECATION_WARNINGS
			bIsMultiViewportEnabled = !bIsSingleViewCapture && Aspects.IsInstancedMultiViewportEnabled();
			PRAGMA_ENABLE_DEPRECATION_WARNINGS
			
			if (Aspects.IsMobileMultiViewEnabled())
			{
				bIsMobileMultiViewEnabled = !bIsSingleViewCapture && Family && Family->bRequireMultiView;
				MultiViewCount = bIsMobileMultiViewEnabled ? 2 : 1;
			}
			
			bIsSinglePassStereo = bIsInstancedStereoEnabled || bIsMobileMultiViewEnabled;
		}
	}
	
	// Stereo setup needs to be done before this as that influences how the culling frustrum is created.
	SetupViewFrustum();

	// Determine whether the view should reverse the cull mode due to a negative determinant.  Only do this for a valid scene
	bReverseCulling = (Family && Family->Scene) ? ViewMatrices.GetWorldToView().Determinant() < 0.0f : false;

	// Setup transformation constants to be used by the graphics hardware to transform device normalized depth samples
	// into world oriented z.
	InvDeviceZToWorldZTransform = CreateInvDeviceZToWorldZTransform(ProjectionMatrixUnadjustedForRHI);

	static TConsoleVariableData<int32>* SortPolicyCvar = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.TranslucentSortPolicy"));
	TranslucentSortPolicy = static_cast<ETranslucentSortPolicy::Type>(SortPolicyCvar->GetValueOnAnyThread());

	TranslucentSortAxis = GetDefault<URendererSettings>()->TranslucentSortAxis;

	// As the world is only accessible from the game thread, bIsGameView should be explicitly
	// set on any other thread.
	if(IsInGameThread())
	{
		bIsGameView = (Family && Family->Scene && Family->Scene->GetWorld() ) ? Family->Scene->GetWorld()->IsGameWorld() : false;
	}

	if(ViewMatrices.IsPerspectiveProjection())
	{
		bUseFieldOfViewForLOD = InitOptions.bUseFieldOfViewForLOD;
		FOV = InitOptions.FOV;
		DesiredFOV = InitOptions.DesiredFOV;
	}
	else
	{
		bUseFieldOfViewForLOD = false;
		FOV = 0.0f;
		DesiredFOV = 0.0f;
	}

	DrawDynamicFlags = EDrawDynamicFlags::None;
	bAllowTemporalJitter = true;

#if WITH_EDITOR
	bUsePixelInspector = false;

	EditorViewBitflag = InitOptions.EditorViewBitflag;

	SelectionOutlineColor = GEngine->GetSelectionOutlineColor();
	SubduedSelectionOutlineColor = GEngine->GetSubduedSelectionOutlineColor();
#endif

	SetupAntiAliasingMethod();

	if (AntiAliasingMethod == AAM_TSR)
	{
		PrimaryScreenPercentageMethod = EPrimaryScreenPercentageMethod::TemporalUpscale;
	}
	else if (AntiAliasingMethod == AAM_TemporalAA && CVarEnableTemporalUpsample.GetValueOnAnyThread() != 0)
	{
		PrimaryScreenPercentageMethod = EPrimaryScreenPercentageMethod::TemporalUpscale;
	}

	if (Family)
	{
		// Find the primary view in the view family, if this is the primary view itself assume it'll be added at the back of the family.
		PrimaryViewIndex = Family->Views.Num();
		if (IStereoRendering::IsASecondaryView(*this))
		{
			check(Family->Views.Num() >= 1);
			while (--PrimaryViewIndex >= 0)
			{
				const FSceneView* PrimaryView = Family->Views[PrimaryViewIndex];
				if (IStereoRendering::IsAPrimaryView(*PrimaryView))
				{
					break;
				}
			}
		}

		if (Family->bResolveScene && Family->EngineShowFlags.PostProcessing)
		{
			EyeAdaptationViewState = State;

			// When rendering in stereo we want to use the same exposure for both eyes.
			if (IStereoRendering::IsASecondaryView(*this))
			{
				check(Family->Views.Num() >= 1);
				const FSceneView* PrimaryView = Family->Views[PrimaryViewIndex];
				if (IStereoRendering::IsAPrimaryView(*PrimaryView))
				{
					EyeAdaptationViewState = PrimaryView->State;
				}
			}
		}

		//  Cube maps need to share eye adaptation to avoid seams.
		if (bIsSceneCaptureCube && Family->EngineShowFlags.PostProcessing && Family->Views.Num())
		{
			EyeAdaptationViewState = Family->Views[0]->State;
		}

		// User specified exposure view state, for sharing exposure across views (for example, for movie render queue high-res tiled rendering)
		if (InitOptions.ExposureSceneViewStateInterface)
		{
			EyeAdaptationViewState = InitOptions.ExposureSceneViewStateInterface;
		}
	}

#if DO_CHECK || USING_CODE_ANALYSIS
	check(VerifyMembersChecks());
#endif
}

FSceneView::~FSceneView() = default;

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
const FViewMatrices& FSceneView::GetShadowViewMatrices() const
{
	return ShadowFreezeCamera::Backup ? *ShadowFreezeCamera::Backup : ViewMatrices;
}
#endif

int32 FSceneView::GetMultiViewCount() const
{
	return MultiViewCount;
}

bool FSceneView::UsesMobileMultiViewShaders() const
{
	return Aspects.IsMobileMultiViewEnabled();
}

bool FSceneView::UsesInstancedStereoShaders() const
{
	return Aspects.IsInstancedStereoEnabled();
}

bool FSceneView::IsInstancedStereo() const
{
	return bIsInstancedStereoEnabled && IStereoRendering::IsStereoEyeView(*this);
}

FRDGTextureDesc FSceneView::CreateStereoTextureDesc(FIntPoint Size, EPixelFormat Format, FClearValueBinding Clear, ETextureCreateFlags Flags) const
{
	return
		UsesMobileMultiViewShaders() ?
		FRDGTextureDesc::Create2DArray(Size, Format, Clear, Flags, MultiViewCount) :
		FRDGTextureDesc::Create2D(Size, Format, Clear, Flags);
}

#if DO_CHECK || USING_CODE_ANALYSIS
bool FSceneView::VerifyMembersChecks() const
{
	bool bIsTemporalAccumulation = IsTemporalAccumulationBasedMethod(AntiAliasingMethod);

	if (PrimaryScreenPercentageMethod == EPrimaryScreenPercentageMethod::TemporalUpscale)
	{
		checkf(bIsTemporalAccumulation, TEXT("ScreenPercentageMethod == EPrimaryScreenPercentageMethod::TemporalUpscale requires TAA, TSR or custom Temporal Upscaler"));
	}

	if (bIsTemporalAccumulation)
	{
		checkf(State, TEXT("TemporalAA requires the view to have a valid state."));
	}

	return true;
}
#endif


void FSceneView::SetupAntiAliasingMethod()
{
	{
		int32 Value = GetDefaultAntiAliasingMethod(FeatureLevel);
		if (Value >= 0 && Value < AAM_MAX)
		{
			AntiAliasingMethod = (EAntiAliasingMethod)Value;
		}
	}

	if (Family)
	{
		const bool bWillApplyTemporalAA = Family->EngineShowFlags.PostProcessing || bIsPlanarReflection;

		if (!bWillApplyTemporalAA || !Family->EngineShowFlags.AntiAliasing)
		{
			AntiAliasingMethod = AAM_None;
		}

		if (AntiAliasingMethod == AAM_TemporalAA)
		{
			if (!Family->EngineShowFlags.TemporalAA || !Family->bRealtimeUpdate || !SupportsGen4TAA(GetShaderPlatform()))
			{
				AntiAliasingMethod = AAM_FXAA;
			}
		}
		else if (AntiAliasingMethod == AAM_TSR)
		{
			// TODO(TSR): Support TSR with bRealtimeUpdate
			if (!Family->EngineShowFlags.TemporalAA || !Family->bRealtimeUpdate || !SupportsTSR(GetShaderPlatform()))
			{
				AntiAliasingMethod = AAM_FXAA;
			}
		}

		checkf(Family->GetTemporalUpscalerInterface() == nullptr, TEXT("ITemporalUpscaler should be set up in FSceneViewExtensionBase::BeginRenderViewFamily()"));
	}

	// TemporalAA requires view state for history.
	if (IsTemporalAccumulationBasedMethod(AntiAliasingMethod) && !State)
	{
		AntiAliasingMethod = AAM_None;
	}
}

FVector FSceneView::GetTemporalLODOrigin(int32 Index, bool bUseLaggedLODTransition) const
{
	if (bUseLaggedLODTransition && State)
	{
		const FTemporalLODState& LODState = State->GetTemporalLODState();
		if (LODState.TemporalLODLag != 0.0f)
		{
			return LODState.TemporalLODViewOrigin[Index];
		}
	}
	return ViewMatrices.GetViewOrigin();
}

float FSceneView::GetTemporalLODTransition() const
{
	if (State)
	{
		return State->GetTemporalLODTransition();
	}
	return 0.0f;
}

uint32 FSceneView::GetDistanceFieldTemporalSampleIndex() const
{
	if (State)
	{
		return State->GetDistanceFieldTemporalSampleIndex();
	}
	return 0;
}

uint32 FSceneView::GetViewKey() const
{
	if (State)
	{
		return State->GetViewKey();
	}
	return 0;
}

uint32 FSceneView::GetOcclusionFrameCounter() const
{
	if (State)
	{
		return State->GetOcclusionFrameCounter();
	}
	return MAX_uint32;
}

void FSceneView::UpdateProjectionMatrix(const FMatrix& NewProjectionMatrix)
{
	ProjectionMatrixUnadjustedForRHI = NewProjectionMatrix;
	InvDeviceZToWorldZTransform = CreateInvDeviceZToWorldZTransform(ProjectionMatrixUnadjustedForRHI);

	// Update init options before creating new view matrices
	SceneViewInitOptions.ProjectionMatrix = NewProjectionMatrix;

	// Create new matrices
	FViewMatrices NewViewMatrices = FViewMatrices(SceneViewInitOptions);
	ViewMatrices = NewViewMatrices;

	SetupViewFrustum();
}

void FViewMatrices::UpdateViewMatrix(const FVector& ViewLocation, const FRotator& ViewRotation)
{
	ViewOrigin = ViewLocation;

	FMatrix ViewPlanesMatrix = FMatrix(
		FPlane(0, 0, 1, 0),
		FPlane(1, 0, 0, 0),
		FPlane(0, 1, 0, 0),
		FPlane(0, 0, 0, 1));

	const FMatrix ViewRotationMatrix = FInverseRotationMatrix(ViewRotation) * ViewPlanesMatrix;

	WorldToView = FTranslationMatrix(-ViewLocation) * ViewRotationMatrix;
	ViewToWorld = ViewRotationMatrix.GetTransposed() * FTranslationMatrix(ViewLocation);

	// Duplicate HMD rotation matrix with roll removed
	FRotator HMDViewRotation = ViewRotation;
	HMDViewRotation.Roll = 0.f;
	HMDViewMatrixNoRoll = FInverseRotationMatrix(HMDViewRotation) * ViewPlanesMatrix;

	PreViewTranslation = -ViewOrigin;

	TranslatedViewMatrix = ViewRotationMatrix;
	InvTranslatedViewMatrix = TranslatedViewMatrix.GetTransposed();

	OverriddenTranslatedViewMatrix = ViewRotationMatrix;
	OverriddenInvTranslatedViewMatrix = OverriddenTranslatedViewMatrix.GetTransposed();;

	RecomputeDerivedMatrices();
}

FMatrix FViewMatrices::ScreenToClipProjectionMatrix() const
{
	//Screen to clip matrix should not utilise scene depth for the w component in ortho projections, but is needed for perspective
	if (IsPerspectiveProjection())
	{
		return FMatrix(
			FPlane(1, 0, 0, 0),
			FPlane(0, 1, 0, 0),
			FPlane(0, 0, ViewToClip.M[2][2], 1.0f),
			FPlane(0, 0, ViewToClip.M[3][2], 0.0f));
	}
	else
	{
		return FMatrix(
			FPlane(1, 0, 0, 0),
			FPlane(0, 1, 0, 0),
			FPlane(0, 0, ViewToClip.M[2][2], 0.0f),
			FPlane(0, 0, ViewToClip.M[3][2], 1.0f));
	}
}

void FViewMatrices::HackOverrideViewMatrixForShadows(const FMatrix& InViewMatrix)
{
	OverriddenTranslatedViewMatrix = InViewMatrix;
	WorldToView = FTranslationMatrix(-ViewOrigin) * InViewMatrix;
	ViewToWorld = WorldToView.Inverse();
	OverriddenInvTranslatedViewMatrix = OverriddenTranslatedViewMatrix.Inverse();
}

void FSceneView::UpdateViewMatrix()
{
	FVector StereoViewLocation = ViewLocation;
	FRotator StereoViewRotation = ViewRotation;
	if (GEngine->StereoRenderingDevice.IsValid() && IStereoRendering::IsStereoEyePass(StereoPass))
	{
		GEngine->StereoRenderingDevice->CalculateStereoViewOffset(StereoViewIndex, StereoViewRotation, WorldToMetersScale, StereoViewLocation);
		ViewLocation = StereoViewLocation;
		ViewRotation = StereoViewRotation;
	}

	ViewMatrices.UpdateViewMatrix(StereoViewLocation, StereoViewRotation);
	GetViewFrustumBounds(ViewFrustum, ViewMatrices.GetWorldToClip(), false);
}

void FViewMatrices::UpdatePlanarReflectionViewMatrix(const FSceneView& SourceView, const FMirrorMatrix& MirrorMatrix)
{
	// This is a subset of the FSceneView ctor that recomputes the transforms changed by late updating the parent camera (in UpdateViewMatrix)
	const FMatrix LocalViewMatrix(MirrorMatrix * SourceView.ViewMatrices.GetWorldToView());
	HMDViewMatrixNoRoll = LocalViewMatrix.RemoveTranslation();

	ViewOrigin = LocalViewMatrix.InverseTransformPosition(FVector::ZeroVector);
	PreViewTranslation = -ViewOrigin;

	WorldToView = FTranslationMatrix(-ViewOrigin) * HMDViewMatrixNoRoll;
	//ViewToWorld = FTranslationMatrix(-WorldToView.GetOrigin()) * WorldToView.RemoveTranslation().GetTransposed();

	ViewToWorld = HMDViewMatrixNoRoll.GetTransposed() * FTranslationMatrix(ViewOrigin);

	WorldToClip = GetWorldToView() * GetViewToClip();
	ClipToWorld = GetClipToView() * ViewToWorld;

	OverriddenTranslatedViewMatrix = TranslatedViewMatrix = HMDViewMatrixNoRoll;
	OverriddenInvTranslatedViewMatrix = InvTranslatedViewMatrix = HMDViewMatrixNoRoll.GetTransposed();

	TranslatedWorldToClip = GetTranslatedViewMatrix() * GetViewToClip();
	ClipToTranslatedWorld = GetClipToView() * GetInvTranslatedViewMatrix();
}

void FSceneView::UpdatePlanarReflectionViewMatrix(const FSceneView& SourceView, const FMirrorMatrix& MirrorMatrix)
{
	ViewMatrices.UpdatePlanarReflectionViewMatrix(SourceView, MirrorMatrix);

	// Update bounds
	GetViewFrustumBounds(ViewFrustum, ViewMatrices.GetWorldToClip(), false);
}

FVector4 FSceneView::WorldToScreen(const FVector& WorldPoint) const
{
	return ViewMatrices.GetWorldToClip().TransformFVector4(FVector4(WorldPoint,1));
}

FVector FSceneView::ScreenToWorld(const FVector4& ScreenPoint) const
{
	return ViewMatrices.GetClipToWorld().TransformFVector4(ScreenPoint);
}

bool FSceneView::ScreenToPixel(const FVector4& ScreenPoint,FVector2D& OutPixelLocation) const
{
	if(ScreenPoint.W != 0.0f)
	{
		//Reverse the W in the case it is negative, this allow to manipulate a manipulator in the same direction when the camera is really close to the manipulator.
		float InvW = (ScreenPoint.W > 0.0f ? 1.0f : -1.0f) / ScreenPoint.W;
		float Y = (GProjectionSignY > 0.0f) ? ScreenPoint.Y : 1.0f - ScreenPoint.Y;
		OutPixelLocation = FVector2D(
			UnscaledViewRect.Min.X + (0.5f + ScreenPoint.X * 0.5f * InvW) * UnscaledViewRect.Width(),
			UnscaledViewRect.Min.Y + (0.5f - Y * 0.5f * InvW) * UnscaledViewRect.Height()
			);
		return true;
	}
	else
	{
		return false;
	}
}

FVector4 FSceneView::PixelToScreen(float InX,float InY,float Z) const
{
	if (GProjectionSignY > 0.0f)
	{
		return FVector4(
			-1.0f + InX / UnscaledViewRect.Width() * +2.0f,
			+1.0f + InY / UnscaledViewRect.Height() * -2.0f,
			Z,
			1
			);
	}
	else
	{
		return FVector4(
			-1.0f + InX / UnscaledViewRect.Width() * +2.0f,
			1.0f - (+1.0f + InY / UnscaledViewRect.Height() * -2.0f),
			Z,
			1
			);
	}
}

// Similar to PixelToScreen, but handles cases where the viewport isn't the whole render target, such as a mouse cursor location in
// render target space.  If desired and outside the bounds of the viewport, will clamp to the nearest edge, so a valid result is returned.
FVector4 FSceneView::CursorToScreen(float InX, float InY, float Z, bool bClamped /* = true */) const
{
	float X = InX - UnscaledViewRect.Min.X;
	float Y = InY - UnscaledViewRect.Min.Y; 

	if (bClamped)
	{
		X = FMath::Clamp(X, 0.0f, UnscaledViewRect.Width());
		Y = FMath::Clamp(Y, 0.0f, UnscaledViewRect.Height());
	}
	
	return PixelToScreen(X, Y, Z);
}

/** Transforms a point from the view's world-space into pixel coordinates relative to the view's X,Y. */
bool FSceneView::WorldToPixel(const FVector& WorldPoint,FVector2D& OutPixelLocation) const
{
	const FVector4 ScreenPoint = WorldToScreen(WorldPoint);
	return ScreenToPixel(ScreenPoint, OutPixelLocation);
}

/** Transforms a point from pixel coordinates relative to the view's X,Y (left, top) into the view's world-space. */
FVector4 FSceneView::PixelToWorld(float X,float Y,float Z) const
{
	const FVector4 ScreenPoint = PixelToScreen(X, Y, Z);
	return ScreenToWorld(ScreenPoint);
}

/**
 * Transforms a point from the view's world-space into the view's screen-space.
 * Divides the resulting X, Y, Z by W before returning.
 */
FPlane FSceneView::Project(const FVector& WorldPoint) const
{
	FPlane Result = WorldToScreen(WorldPoint);

	if (Result.W == 0)
	{
		Result.W = UE_KINDA_SMALL_NUMBER;
	}

	const float RHW = 1.0f / Result.W;

	return FPlane(Result.X * RHW,Result.Y * RHW,Result.Z * RHW,Result.W);
}

/**
 * Transforms a point from the view's screen-space into world coordinates
 * multiplies X, Y, Z by W before transforming.
 */
FVector FSceneView::Deproject(const FPlane& ScreenPoint) const
{
	return ViewMatrices.GetClipToWorld().TransformFVector4(FPlane(ScreenPoint.X * ScreenPoint.W,ScreenPoint.Y * ScreenPoint.W,ScreenPoint.Z * ScreenPoint.W,ScreenPoint.W));
}

void FSceneView::DeprojectFVector2D(const FVector2D& ScreenPos, FVector& out_WorldOrigin, FVector& out_WorldDirection) const
{
	const FMatrix InvViewProjectionMatrix = ViewMatrices.GetClipToWorld();
	DeprojectScreenToWorld(ScreenPos, UnscaledViewRect, InvViewProjectionMatrix, out_WorldOrigin, out_WorldDirection);
}

void FSceneView::DeprojectScreenToWorld(const FVector2D& ScreenPos, const FIntRect& ViewRect, const FMatrix& InvViewMatrix, const FMatrix& InvProjectionMatrix, FVector& out_WorldOrigin, FVector& out_WorldDirection)
{
	int32 PixelX = FMath::TruncToInt(ScreenPos.X);
	int32 PixelY = FMath::TruncToInt(ScreenPos.Y);

	// Get the eye position and direction of the mouse cursor in two stages (inverse transform projection, then inverse transform view).
	// This avoids the numerical instability that occurs when a view matrix with large translation is composed with a projection matrix

	// Get the pixel coordinates into 0..1 normalized coordinates within the constrained view rectangle
	const float NormalizedX = (PixelX - ViewRect.Min.X) / ((float)ViewRect.Width());
	const float NormalizedY = (PixelY - ViewRect.Min.Y) / ((float)ViewRect.Height());

	// Get the pixel coordinates into -1..1 projection space
	const float ScreenSpaceX = (NormalizedX - 0.5f) * 2.0f;
	const float ScreenSpaceY = ((1.0f - NormalizedY) - 0.5f) * 2.0f;

	// The start of the ray trace is defined to be at mousex,mousey,1 in projection space (z=1 is near, z=0 is far - this gives us better precision)
	// To get the direction of the ray trace we need to use any z between the near and the far plane, so let's use (mousex, mousey, 0.01)
	const FVector4 RayStartProjectionSpace = FVector4(ScreenSpaceX, ScreenSpaceY, 1.0f, 1.0f);
	const FVector4 RayEndProjectionSpace = FVector4(ScreenSpaceX, ScreenSpaceY, 0.01f, 1.0f);

	// Projection (changing the W coordinate) is not handled by the FMatrix transforms that work with vectors, so multiplications
	// by the projection matrix should use homogeneous coordinates (i.e. FPlane).
	const FVector4 HGRayStartViewSpace = InvProjectionMatrix.TransformFVector4(RayStartProjectionSpace);
	const FVector4 HGRayEndViewSpace = InvProjectionMatrix.TransformFVector4(RayEndProjectionSpace);
	FVector RayStartViewSpace(HGRayStartViewSpace.X, HGRayStartViewSpace.Y, HGRayStartViewSpace.Z);
	FVector RayEndViewSpace(HGRayEndViewSpace.X,   HGRayEndViewSpace.Y,   HGRayEndViewSpace.Z);
	// divide vectors by W to undo any projection and get the 3-space coordinate
	if (HGRayStartViewSpace.W != 0.0f)
	{
		RayStartViewSpace /= HGRayStartViewSpace.W;
	}
	if (HGRayEndViewSpace.W != 0.0f)
	{
		RayEndViewSpace /= HGRayEndViewSpace.W;
	}
	FVector RayDirViewSpace = RayEndViewSpace - RayStartViewSpace;
	RayDirViewSpace = RayDirViewSpace.GetSafeNormal();

	// The view transform does not have projection, so we can use the standard functions that deal with vectors and normals (normals
	// are vectors that do not use the translational part of a rotation/translation)
	const FVector RayStartWorldSpace = InvViewMatrix.TransformPosition(RayStartViewSpace);
	const FVector RayDirWorldSpace = InvViewMatrix.TransformVector(RayDirViewSpace);

	// Finally, store the results in the hitcheck inputs.  The start position is the eye, and the end position
	// is the eye plus a long distance in the direction the mouse is pointing.
	out_WorldOrigin = RayStartWorldSpace;
	out_WorldDirection = RayDirWorldSpace.GetSafeNormal();
}

void FSceneView::DeprojectScreenToWorld(const FVector2D& ScreenPos, const FIntRect& ViewRect, const FMatrix& InvViewProjMatrix, FVector& out_WorldOrigin, FVector& out_WorldDirection)
{
	float PixelX = FMath::TruncToFloat(ScreenPos.X);
	float PixelY = FMath::TruncToFloat(ScreenPos.Y);

	// Get the eye position and direction of the mouse cursor in two stages (inverse transform projection, then inverse transform view).
	// This avoids the numerical instability that occurs when a view matrix with large translation is composed with a projection matrix

	// Get the pixel coordinates into 0..1 normalized coordinates within the constrained view rectangle
	const float NormalizedX = (PixelX - ViewRect.Min.X) / ((float)ViewRect.Width());
	const float NormalizedY = (PixelY - ViewRect.Min.Y) / ((float)ViewRect.Height());

	// Get the pixel coordinates into -1..1 projection space
	const float ScreenSpaceX = (NormalizedX - 0.5f) * 2.0f;
	const float ScreenSpaceY = ((1.0f - NormalizedY) - 0.5f) * 2.0f;

	// The start of the ray trace is defined to be at mousex,mousey,1 in projection space (z=1 is near, z=0 is far - this gives us better precision)
	// To get the direction of the ray trace we need to use any z between the near and the far plane, so let's use (mousex, mousey, 0.01)
	const FVector4 RayStartProjectionSpace = FVector4(ScreenSpaceX, ScreenSpaceY, 1.0f, 1.0f);
	const FVector4 RayEndProjectionSpace = FVector4(ScreenSpaceX, ScreenSpaceY, 0.01f, 1.0f);

	// Projection (changing the W coordinate) is not handled by the FMatrix transforms that work with vectors, so multiplications
	// by the projection matrix should use homogeneous coordinates (i.e. FPlane).
	const FVector4 HGRayStartWorldSpace = InvViewProjMatrix.TransformFVector4(RayStartProjectionSpace);
	const FVector4 HGRayEndWorldSpace = InvViewProjMatrix.TransformFVector4(RayEndProjectionSpace);
	FVector RayStartWorldSpace(HGRayStartWorldSpace.X, HGRayStartWorldSpace.Y, HGRayStartWorldSpace.Z);
	FVector RayEndWorldSpace(HGRayEndWorldSpace.X, HGRayEndWorldSpace.Y, HGRayEndWorldSpace.Z);
	// divide vectors by W to undo any projection and get the 3-space coordinate
	if (HGRayStartWorldSpace.W != 0.0f)
	{
		RayStartWorldSpace /= HGRayStartWorldSpace.W;
	}
	if (HGRayEndWorldSpace.W != 0.0f)
	{
		RayEndWorldSpace /= HGRayEndWorldSpace.W;
	}
	const FVector RayDirWorldSpace = (RayEndWorldSpace - RayStartWorldSpace).GetSafeNormal();

	// Finally, store the results in the outputs
	out_WorldOrigin = RayStartWorldSpace;
	out_WorldDirection = RayDirWorldSpace;
}

bool FSceneView::ProjectWorldToScreen(const FVector& WorldPosition, const FIntRect& ViewRect, const FMatrix& ViewProjectionMatrix, FVector2D& out_ScreenPos, bool bShouldCalcOutsideViewPosition /*= false*/)
{
	FPlane Result = ViewProjectionMatrix.TransformFVector4(FVector4(WorldPosition, 1.f));
	bool bIsInsideView = Result.W > 0.0f;
	double W = Result.W;

	// If WorldPosition is outside the ViewProjectionMatrix and we don't force to calc the outside view position, stop the calcs.
	if (!bIsInsideView && !bShouldCalcOutsideViewPosition)
	{
		return false;
	}

	// Tweak our W value to allow the outside view position calcs if the variable is enabled.
	if (bShouldCalcOutsideViewPosition)
	{
		W = FMath::Abs(Result.W);
	}
	
	// the result of this will be x and y coords in -1..1 projection space
	const float RHW = 1.0f / W;
	FPlane PosInScreenSpace = FPlane(Result.X * RHW, Result.Y * RHW, Result.Z * RHW, W);

	// Move from projection space to normalized 0..1 UI space
	const float NormalizedX = ( PosInScreenSpace.X / 2.f ) + 0.5f;
	const float NormalizedY = 1.f - ( PosInScreenSpace.Y / 2.f ) - 0.5f;

	FVector2D RayStartViewRectSpace(
		( NormalizedX * (float)ViewRect.Width() ),
		( NormalizedY * (float)ViewRect.Height() )
		);

	out_ScreenPos = RayStartViewRectSpace + FVector2D(static_cast<float>(ViewRect.Min.X), static_cast<float>(ViewRect.Min.Y));

	return bIsInsideView;
}


#define LERP_PP(NAME) if(Src.bOverride_ ## NAME)	Dest . NAME = FMath::Lerp(Dest . NAME, Src . NAME, Weight);
#define SET_PP(NAME)  if(Src.bOverride_ ## NAME)    Dest . NAME = Src . NAME;
#define IF_PP(NAME)   if(Src.bOverride_ ## NAME && Src . NAME)

// @param Weight 0..1
void FSceneView::OverridePostProcessSettings(const FPostProcessSettings& Src, float Weight, bool bAllowBlendables)
{
	SCOPE_CYCLE_COUNTER(STAT_OverridePostProcessSettings);

	if(Weight <= 0.0f)
	{
		// no need to blend anything
		return;
	}

	if(Weight > 1.0f)
	{
		Weight = 1.0f;
	}

	{
		FFinalPostProcessSettings& Dest = FinalPostProcessSettings;

		// The following code needs to be adjusted when settings in FPostProcessSettings change.
		SET_PP(TemperatureType);
		LERP_PP(WhiteTemp);
		LERP_PP(WhiteTint);

		LERP_PP(ColorSaturation);
		LERP_PP(ColorContrast);
		LERP_PP(ColorGamma);
		LERP_PP(ColorGain);
		LERP_PP(ColorOffset);

		LERP_PP(ColorSaturationShadows);
		LERP_PP(ColorContrastShadows);
		LERP_PP(ColorGammaShadows);
		LERP_PP(ColorGainShadows);
		LERP_PP(ColorOffsetShadows);

		LERP_PP(ColorSaturationMidtones);
		LERP_PP(ColorContrastMidtones);
		LERP_PP(ColorGammaMidtones);
		LERP_PP(ColorGainMidtones);
		LERP_PP(ColorOffsetMidtones);

		LERP_PP(ColorSaturationHighlights);
		LERP_PP(ColorContrastHighlights);
		LERP_PP(ColorGammaHighlights);
		LERP_PP(ColorGainHighlights);
		LERP_PP(ColorOffsetHighlights);

		LERP_PP(ColorCorrectionShadowsMax);
		LERP_PP(ColorCorrectionHighlightsMin);
		LERP_PP(ColorCorrectionHighlightsMax);

		LERP_PP(BlueCorrection);
		LERP_PP(ExpandGamut);
		LERP_PP(ToneCurveAmount);

		SET_PP(TonemappingMethod);
		LERP_PP(FilmSlope);
		LERP_PP(FilmToe);
		LERP_PP(FilmShoulder);
		LERP_PP(FilmBlackClip);
		LERP_PP(FilmWhiteClip);

		LERP_PP(SceneColorTint);
		LERP_PP(SceneFringeIntensity);
		LERP_PP(ChromaticAberrationStartOffset);
		LERP_PP(BloomIntensity);
		LERP_PP(BloomGaussianIntensity);
		LERP_PP(BloomThreshold);
		LERP_PP(Bloom1Tint);
		LERP_PP(BloomSizeScale);
		LERP_PP(Bloom1Size);
		LERP_PP(Bloom2Tint);
		LERP_PP(Bloom2Size);
		LERP_PP(Bloom3Tint);
		LERP_PP(Bloom3Size);
		LERP_PP(Bloom4Tint);
		LERP_PP(Bloom4Size);
		LERP_PP(Bloom5Tint);
		LERP_PP(Bloom5Size);
		LERP_PP(Bloom6Tint);
		LERP_PP(Bloom6Size);
		LERP_PP(BloomDirtMaskIntensity);
		LERP_PP(BloomDirtMaskTint);
		LERP_PP(BloomConvolutionIntensity);
		LERP_PP(BloomConvolutionScatterDispersion);
		LERP_PP(BloomConvolutionSize);
		LERP_PP(BloomConvolutionCenterUV);
		LERP_PP(BloomConvolutionPreFilterMin);
		LERP_PP(BloomConvolutionPreFilterMax);
		LERP_PP(BloomConvolutionPreFilterMult);
		SET_PP(bMegaLights);
		SET_PP(MegaLightsFrontLayerTranslucency);
		LERP_PP(AmbientCubemapIntensity);
		LERP_PP(AmbientCubemapTint);
		LERP_PP(CameraShutterSpeed);
		LERP_PP(CameraISO);
		LERP_PP(AutoExposureLowPercent);
		LERP_PP(AutoExposureHighPercent);
		LERP_PP(AutoExposureMinBrightness);
		LERP_PP(AutoExposureMaxBrightness);
		LERP_PP(AutoExposureSpeedUp);
		LERP_PP(AutoExposureSpeedDown);
		LERP_PP(AutoExposureBias);
		LERP_PP(HistogramLogMin);
		LERP_PP(HistogramLogMax);
		SET_PP(LocalExposureMethod);
		LERP_PP(LocalExposureContrastScale_DEPRECATED);
		LERP_PP(LocalExposureHighlightContrastScale);
		LERP_PP(LocalExposureShadowContrastScale);
		LERP_PP(LocalExposureHighlightThreshold);
		LERP_PP(LocalExposureShadowThreshold);
		LERP_PP(LocalExposureDetailStrength);
		LERP_PP(LocalExposureBlurredLuminanceBlend);
		LERP_PP(LocalExposureBlurredLuminanceKernelSizePercent);
		LERP_PP(LocalExposureHighlightThresholdStrength);
		LERP_PP(LocalExposureShadowThresholdStrength);
		LERP_PP(LocalExposureMiddleGreyBias);
		LERP_PP(LensFlareIntensity);
		LERP_PP(LensFlareTint);
		LERP_PP(LensFlareBokehSize);
		LERP_PP(LensFlareThreshold);
		LERP_PP(VignetteIntensity);
		SET_PP(VignetteType);
		LERP_PP(VignetteCenter);
		LERP_PP(VignetteColor);
		LERP_PP(VignetteSize);
		LERP_PP(VignetteSoftness);
		SET_PP(VignetteTexture);
		LERP_PP(Sharpen);
		LERP_PP(FilmGrainIntensity);
		LERP_PP(FilmGrainIntensityShadows);
		LERP_PP(FilmGrainIntensityMidtones);
		LERP_PP(FilmGrainIntensityHighlights);
		LERP_PP(FilmGrainShadowsMax);
		LERP_PP(FilmGrainHighlightsMin);
		LERP_PP(FilmGrainHighlightsMax);
		LERP_PP(FilmGrainScale);
		LERP_PP(AmbientOcclusionIntensity);
		LERP_PP(AmbientOcclusionStaticFraction);
		LERP_PP(AmbientOcclusionRadius);
		LERP_PP(AmbientOcclusionFadeDistance);
		LERP_PP(AmbientOcclusionFadeRadius);
		LERP_PP(AmbientOcclusionDistance_DEPRECATED);
		LERP_PP(AmbientOcclusionPower);
		LERP_PP(AmbientOcclusionBias);
		LERP_PP(AmbientOcclusionQuality);
		LERP_PP(AmbientOcclusionMipBlend);
		LERP_PP(AmbientOcclusionMipScale);
		LERP_PP(AmbientOcclusionMipThreshold);
		LERP_PP(AmbientOcclusionTemporalBlendWeight);
		LERP_PP(IndirectLightingColor);
		LERP_PP(IndirectLightingIntensity);

		if (Src.bOverride_DepthOfFieldFocalDistance)
		{
			if (Dest.DepthOfFieldFocalDistance == 0.0f || Src.DepthOfFieldFocalDistance == 0.0f)
			{
				Dest.DepthOfFieldFocalDistance = Src.DepthOfFieldFocalDistance;
			}
			else
			{
				Dest.DepthOfFieldFocalDistance = FMath::Lerp(Dest.DepthOfFieldFocalDistance, Src.DepthOfFieldFocalDistance, Weight);
			}
		}
		LERP_PP(DepthOfFieldFstop);
		LERP_PP(DepthOfFieldMinFstop);
		SET_PP(DepthOfFieldEnabled)
		LERP_PP(DepthOfFieldSensorWidth);
		LERP_PP(DepthOfFieldSqueezeFactor);
		LERP_PP(DepthOfFieldDepthBlurRadius);
		SET_PP(DepthOfFieldUseHairDepth)
		LERP_PP(DepthOfFieldPetzvalBokeh);
		LERP_PP(DepthOfFieldPetzvalBokehFalloff);
		LERP_PP(DepthOfFieldPetzvalExclusionBoxExtents);
		LERP_PP(DepthOfFieldPetzvalExclusionBoxRadius);
		LERP_PP(DepthOfFieldBarrelRadius);
		LERP_PP(DepthOfFieldBarrelLength);
		if (Src.bOverride_DepthOfFieldMatteBoxFlags)
		{
			for (uint32 Index = 0; Index < UE_ARRAY_COUNT(Src.DepthOfFieldMatteBoxFlags); ++Index)
			{
				const FMatteBoxFlag& SrcFlag = Src.DepthOfFieldMatteBoxFlags[Index];
				FMatteBoxFlag& DestFlag = Dest.DepthOfFieldMatteBoxFlags[Index];
				DestFlag.Pitch = FMath::Lerp(DestFlag.Pitch, SrcFlag.Pitch, Weight);
				DestFlag.Roll = FMath::Lerp(DestFlag.Roll, SrcFlag.Roll, Weight);
				DestFlag.Length = FMath::Lerp(DestFlag.Length, SrcFlag.Length, Weight);
			}
		}
		LERP_PP(DepthOfFieldDepthBlurAmount);
		LERP_PP(DepthOfFieldFocalRegion);
		LERP_PP(DepthOfFieldNearTransitionRegion);
		LERP_PP(DepthOfFieldFarTransitionRegion);
		LERP_PP(DepthOfFieldScale);
		LERP_PP(DepthOfFieldNearBlurSize);
		LERP_PP(DepthOfFieldFarBlurSize);
		LERP_PP(DepthOfFieldOcclusion);
		LERP_PP(DepthOfFieldSkyFocusDistance);
		LERP_PP(DepthOfFieldVignetteSize);
		LERP_PP(DepthOfFieldAspectRatioScalar);
		LERP_PP(MotionBlurAmount);
		LERP_PP(MotionBlurMax);
		LERP_PP(MotionBlurPerObjectSize);
		LERP_PP(ScreenSpaceReflectionQuality);
		LERP_PP(ScreenSpaceReflectionIntensity);
		LERP_PP(ScreenSpaceReflectionMaxRoughness);

		SET_PP(TranslucencyType);
		SET_PP(RayTracingTranslucencyMaxRoughness);
		SET_PP(RayTracingTranslucencyRefractionRays);
		SET_PP(RayTracingTranslucencySamplesPerPixel);
		SET_PP(RayTracingTranslucencyShadows);
		SET_PP(RayTracingTranslucencyRefraction);
		SET_PP(RayTracingTranslucencyMaxPrimaryHitEvents);
		SET_PP(RayTracingTranslucencyMaxSecondaryHitEvents);
		SET_PP(RayTracingTranslucencyUseRayTracedRefraction);

		SET_PP(DynamicGlobalIlluminationMethod);
		SET_PP(LumenSurfaceCacheResolution);
		SET_PP(LumenSceneLightingQuality);
		SET_PP(LumenSceneDetail);
		SET_PP(LumenSceneViewDistance);
		SET_PP(LumenSceneLightingUpdateSpeed);
		SET_PP(LumenFinalGatherQuality);
		SET_PP(LumenFinalGatherLightingUpdateSpeed);
		SET_PP(LumenFinalGatherScreenTraces);
		SET_PP(LumenMaxTraceDistance);

		LERP_PP(LumenDiffuseColorBoost);
		LERP_PP(LumenAmbientOcclusionIntensity);
		LERP_PP(LumenSkylightLeaking);
		LERP_PP(LumenSkylightLeakingTint);
		LERP_PP(LumenFullSkylightLeakingDistance);

		SET_PP(LumenRayLightingMode);
		SET_PP(LumenReflectionsScreenTraces);
		SET_PP(LumenFrontLayerTranslucencyReflections);
		SET_PP(LumenMaxRoughnessToTraceReflections);
		SET_PP(LumenMaxReflectionBounces);
		SET_PP(LumenMaxRefractionBounces);
		SET_PP(ReflectionMethod);
		SET_PP(LumenReflectionQuality);
		SET_PP(RayTracingAO);
		SET_PP(RayTracingAOSamplesPerPixel);
		SET_PP(RayTracingAOIntensity);
		SET_PP(RayTracingAORadius);

		// Path Tracing related settings
		SET_PP(PathTracingMaxBounces);
		SET_PP(PathTracingSamplesPerPixel);
		LERP_PP(PathTracingMaxPathIntensity);
		SET_PP(PathTracingEnableEmissiveMaterials);
		SET_PP(PathTracingEnableReferenceDOF);
		SET_PP(PathTracingEnableReferenceAtmosphere);
		SET_PP(PathTracingEnableDenoiser);
		SET_PP(PathTracingIncludeEmissive);
		SET_PP(PathTracingIncludeDiffuse);
		SET_PP(PathTracingIncludeIndirectDiffuse);
		SET_PP(PathTracingIncludeSpecular);
		SET_PP(PathTracingIncludeIndirectSpecular);
		SET_PP(PathTracingIncludeVolume);
		SET_PP(PathTracingIncludeIndirectVolume);

		SET_PP(DepthOfFieldBladeCount);

		// cubemaps are getting blended additively - in contrast to other properties, maybe we should make that consistent
		if (Src.AmbientCubemap && Src.bOverride_AmbientCubemapIntensity)
		{
			FFinalPostProcessSettings::FCubemapEntry Entry;

			Entry.AmbientCubemapTintMulScaleValue = FLinearColor(1, 1, 1, 1) * Src.AmbientCubemapIntensity;

			if (Src.bOverride_AmbientCubemapTint)
			{
				Entry.AmbientCubemapTintMulScaleValue *= Src.AmbientCubemapTint;
			}

			Entry.AmbientCubemap = Src.AmbientCubemap;
			Dest.UpdateEntry(Entry, Weight);
		}

		IF_PP(ColorGradingLUT)
		{
			float ColorGradingIntensity = FMath::Clamp(Src.ColorGradingIntensity, 0.0f, 1.0f);
			Dest.LerpTo(Src.ColorGradingLUT, ColorGradingIntensity * Weight);
		}

		// actual texture cannot be blended but the intensity can be blended
		IF_PP(BloomDirtMask)
		{
			Dest.BloomDirtMask = Src.BloomDirtMask;
		}

		SET_PP(BloomMethod);

		// actual texture cannot be blended but the intensity can be blended
		IF_PP(BloomConvolutionTexture)
		{
			Dest.BloomConvolutionTexture = Src.BloomConvolutionTexture;
		}

		// actual texture cannot be blended but the film grain intensity can be blended
		IF_PP(FilmGrainTexture)
		{
			Dest.FilmGrainTexture = Src.FilmGrainTexture;
		}

		// A continuous blending of this value would result trashing the pre-convolved bloom kernel cache.
		IF_PP(BloomConvolutionBufferScale)
		{
			Dest.BloomConvolutionBufferScale = Src.BloomConvolutionBufferScale;
		}

		// Curve assets can not be blended.
		IF_PP(AutoExposureBiasCurve)
		{
			Dest.AutoExposureBiasCurve = Src.AutoExposureBiasCurve;
		}

		// Texture asset isn't blended
		IF_PP(AutoExposureMeterMask)
		{
			Dest.AutoExposureMeterMask = Src.AutoExposureMeterMask;
		}

		// Curve assets cannot be blended.
		IF_PP(LocalExposureHighlightContrastCurve)
		{
			Dest.LocalExposureHighlightContrastCurve = Src.LocalExposureHighlightContrastCurve;
		}

		// Curve assets cannot be blended.
		IF_PP(LocalExposureShadowContrastCurve)
		{
			Dest.LocalExposureShadowContrastCurve = Src.LocalExposureShadowContrastCurve;
		}

		// actual texture cannot be blended but the intensity can be blended
		IF_PP(LensFlareBokehShape)
		{
			Dest.LensFlareBokehShape = Src.LensFlareBokehShape;
		}

		if (Src.bOverride_LensFlareTints)
		{
			for (uint32 i = 0; i < 8; ++i)
			{
				Dest.LensFlareTints[i] = FMath::Lerp(Dest.LensFlareTints[i], Src.LensFlareTints[i], Weight);
			}
		}

		if (Src.bOverride_MobileHQGaussian)
		{
			Dest.bMobileHQGaussian = Src.bMobileHQGaussian;
		}

		SET_PP(AutoExposureMethod);

		SET_PP(AmbientOcclusionRadiusInWS);

		SET_PP(MotionBlurTargetFPS);

		SET_PP(AutoExposureApplyPhysicalCameraExposure);

		SET_PP(UserFlags);
	}

	// Blendable objects
	if (bAllowBlendables)
	{
		uint32 Count = Src.WeightedBlendables.Array.Num();

		for(uint32 i = 0; i < Count; ++i)
		{
			UObject* Object = Src.WeightedBlendables.Array[i].Object;

			if(!Object || !Object->IsValidLowLevel())
			{
				continue;
			}

			IBlendableInterface* BlendableInterface = Cast<IBlendableInterface>(Object);

			if(!BlendableInterface)
			{
				continue;
			}

			float LocalWeight = FMath::Min(1.0f, Src.WeightedBlendables.Array[i].Weight) * Weight;

			if(LocalWeight > 0.0f)
			{
				BlendableInterface->OverrideBlendableSettings(*this, LocalWeight);
			}
		}

#if WITH_EDITOR
		if (Src.PreviewBlendable)
		{
			FinalPostProcessSettings.PreviewBlendable = Src.PreviewBlendable;
		}
#endif
	}
}

/** Dummy class needed to support Cast<IBlendableInterface>(Object) */
UBlendableInterface::UBlendableInterface(const FObjectInitializer& ObjectInitializer) : Super(ObjectInitializer)
{
}



void FSceneView::StartFinalPostprocessSettings(FVector InViewLocation)
{
	SCOPE_CYCLE_COUNTER(STAT_StartFinalPostprocessSettings);

	check(IsInGameThread());

	// The final settings for the current viewer position (blended together from many volumes).
	// Setup by the main thread, passed to the render thread and never touched again by the main thread.

	// Set values before any override happens.
	FinalPostProcessSettings.SetBaseValues();

	// project settings might want to have different defaults
	{
		if(!CVarDefaultBloom.GetValueOnGameThread())
		{
			FinalPostProcessSettings.BloomIntensity = 0;
		}
		if (!CVarDefaultAmbientOcclusion.GetValueOnGameThread())
		{
			FinalPostProcessSettings.AmbientOcclusionIntensity = 0;
		}
		if (!CVarDefaultAutoExposure.GetValueOnGameThread())
		{
			FinalPostProcessSettings.AutoExposureMinBrightness = 1;
			FinalPostProcessSettings.AutoExposureMaxBrightness = 1;
			if (CVarDefaultAutoExposureExtendDefaultLuminanceRange.GetValueOnGameThread())
			{
				const float MaxLuminance = 1.2f; // Should we use const LuminanceMaxFromLensAttenuation() instead?
				FinalPostProcessSettings.AutoExposureMinBrightness = LuminanceToEV100(MaxLuminance, FinalPostProcessSettings.AutoExposureMinBrightness);
				FinalPostProcessSettings.AutoExposureMaxBrightness = LuminanceToEV100(MaxLuminance, FinalPostProcessSettings.AutoExposureMaxBrightness);
			}
		}
		else
		{
			int32 Value = CVarDefaultAutoExposureMethod.GetValueOnGameThread();
			if (Value >= 0 && Value < AEM_MAX)
			{
				FinalPostProcessSettings.AutoExposureMethod = (EAutoExposureMethod)Value;
			}
		}

		{
			const float HighlightContrast = FMath::Clamp(CVarDefaultLocalExposureHighlightContrast.GetValueOnGameThread(), 0.0f, 1.0f);

			FinalPostProcessSettings.LocalExposureHighlightContrastScale = HighlightContrast;
		}

		{
			const float ShadowContrast = FMath::Clamp(CVarDefaultLocalExposureShadowContrast.GetValueOnGameThread(), 0.0f, 1.0f);

			FinalPostProcessSettings.LocalExposureShadowContrastScale = ShadowContrast;
		}

		if (!CVarDefaultMotionBlur.GetValueOnGameThread())
		{
			FinalPostProcessSettings.MotionBlurAmount = 0;
		}
		if (!CVarDefaultLensFlare.GetValueOnGameThread())
		{
			FinalPostProcessSettings.LensFlareIntensity = 0;
		}

		{
			int32 Value = CVarDefaultAmbientOcclusionStaticFraction.GetValueOnGameThread();

			if(!Value)
			{
				FinalPostProcessSettings.AmbientOcclusionStaticFraction = 0.0f;
			}
		}

		{
			static const auto CVar = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.MegaLights.EnableForProject"));
			FinalPostProcessSettings.bMegaLights = CVar->GetValueOnGameThread() != 0;
		}

		{
			static const auto CVar = IConsoleManager::Get().FindTConsoleVariableDataBool(TEXT("r.MegaLights.FrontLayerTranslucency.EnableForProject"));
			FinalPostProcessSettings.MegaLightsFrontLayerTranslucency = CVar->GetValueOnGameThread();
		}

		{
			static const auto CVar = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.DynamicGlobalIlluminationMethod"));
			FinalPostProcessSettings.DynamicGlobalIlluminationMethod = (EDynamicGlobalIlluminationMethod::Type)CVar->GetValueOnGameThread();
		}

		{
			static const auto CVar = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.ReflectionMethod"));
			FinalPostProcessSettings.ReflectionMethod = (EReflectionMethod::Type)CVar->GetValueOnGameThread();
		}

		{
			static const auto CVar = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.Lumen.TranslucencyReflections.FrontLayer.EnableForProject"));
			FinalPostProcessSettings.LumenFrontLayerTranslucencyReflections = CVar->GetValueOnGameThread() != 0;
		}

		FinalPostProcessSettings.DepthOfFieldUseHairDepth = GetHairStrandsDepthOfFieldUseHairDepth();
	}

	{
		if (GEngine->StereoRenderingDevice.IsValid())
		{
			GEngine->StereoRenderingDevice->StartFinalPostprocessSettings(&FinalPostProcessSettings, StereoPass, StereoViewIndex);
		}
	}

	if (State != nullptr)
	{
		State->OnStartPostProcessing(*this);
	}

	UWorld* World = ((Family != nullptr) && (Family->Scene != nullptr)) ? Family->Scene->GetWorld() : nullptr;

	// Some views have no world (e.g. material preview)
	if (World != nullptr)
	{
		World->AddPostProcessingSettings(InViewLocation, this);
	}
}

void FSceneView::EndFinalPostprocessSettings(const FSceneViewInitOptions& ViewInitOptions)
{
	const auto SceneViewFeatureLevel = GetFeatureLevel();

	{
		static const auto SceneColorFringeQualityCVar = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.SceneColorFringeQuality"));

		int32 FringeQuality = SceneColorFringeQualityCVar->GetValueOnGameThread();
		if (FringeQuality <= 0)
		{
			FinalPostProcessSettings.SceneFringeIntensity = 0;
		}
	}

	{
		static const auto LocalExposureCVar = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.LocalExposure"));

		const int LocalExposureCVarValue = LocalExposureCVar->GetValueOnGameThread();

		if (LocalExposureCVarValue <= 0 || !Family->EngineShowFlags.LocalExposure)
		{
			FinalPostProcessSettings.LocalExposureHighlightContrastScale = 1.0f;
			FinalPostProcessSettings.LocalExposureShadowContrastScale = 1.0f;
			FinalPostProcessSettings.LocalExposureHighlightContrastCurve = nullptr;
			FinalPostProcessSettings.LocalExposureShadowContrastCurve = nullptr;
			FinalPostProcessSettings.LocalExposureDetailStrength = 1.0f;
		}
	}

	{
		static const auto BloomQualityCVar = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.BloomQuality"));

		int Value = BloomQualityCVar->GetValueOnGameThread();

		if(Value <= 0)
		{
			FinalPostProcessSettings.BloomIntensity = 0.0f;
		}
	}

	if(!Family->EngineShowFlags.Bloom)
	{
		FinalPostProcessSettings.BloomIntensity = 0.0f;
	}

	// scale down tone mapper shader permutation
	{
		int32 Quality = CVarTonemapperQuality.GetValueOnGameThread();

		if(Quality < 2)
		{
			FinalPostProcessSettings.VignetteIntensity = 0;
		}

		if(Quality < 4)
		{
			FinalPostProcessSettings.FilmGrainIntensity = 0;
		}
	}

	{
		static const auto DepthOfFieldQualityCVar = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.DepthOfFieldQuality"));

		int Value = DepthOfFieldQualityCVar->GetValueOnGameThread();

		if(Value <= 0)
		{
			FinalPostProcessSettings.DepthOfFieldScale = 0.0f;
		}
	}

	if(!Family->EngineShowFlags.DepthOfField)
	{
		FinalPostProcessSettings.DepthOfFieldScale = 0;
	}

	if(!Family->EngineShowFlags.Vignette)
	{
		FinalPostProcessSettings.VignetteIntensity = 0;
	}

	if(!Family->EngineShowFlags.Grain)
	{
		FinalPostProcessSettings.FilmGrainIntensity = 0.0f;
	}

	if(!Family->EngineShowFlags.CameraImperfections)
	{
		FinalPostProcessSettings.BloomDirtMaskIntensity = 0;
	}

	if(!Family->EngineShowFlags.AmbientCubemap)
	{
		FinalPostProcessSettings.ContributingCubemaps.Reset();
	}

	if(!Family->EngineShowFlags.LensFlares)
	{
		FinalPostProcessSettings.LensFlareIntensity = 0;
	}

	if (!Family->EngineShowFlags.ToneCurve)
	{
		FinalPostProcessSettings.ToneCurveAmount = 0;
	}

	{
		float Value = CVarExposureOffset.GetValueOnGameThread();
		FinalPostProcessSettings.AutoExposureBias += Value;
	}

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	{
		float& DepthBlurAmount = FinalPostProcessSettings.DepthOfFieldDepthBlurAmount;

		float CVarAmount = CVarDepthOfFieldDepthBlurAmount.GetValueOnGameThread();

		DepthBlurAmount = (CVarAmount > 0.0f) ? (DepthBlurAmount * CVarAmount) : -CVarAmount;
	}

	{
		float& DepthBlurRadius = FinalPostProcessSettings.DepthOfFieldDepthBlurRadius;
		{
			float CVarResScale = FMath::Max(1.0f, CVarDepthOfFieldDepthBlurResolutionScale.GetValueOnGameThread());

			float Factor = FMath::Max(UnscaledViewRect.Width() / 1920.0f - 1.0f, 0.0f);

			DepthBlurRadius *= 1.0f + Factor * (CVarResScale - 1.0f);
		}
		{
			float CVarScale = CVarDepthOfFieldDepthBlurScale.GetValueOnGameThread();

			DepthBlurRadius = (CVarScale > 0.0f) ? (DepthBlurRadius * CVarScale) : -CVarScale;
		}
	}
#endif

	{
		if (GEngine->StereoRenderingDevice.IsValid())
		{
			GEngine->StereoRenderingDevice->EndFinalPostprocessSettings(&FinalPostProcessSettings, StereoPass, StereoViewIndex);
		}
	}

	{
		float Value = CVarSSRMaxRoughness.GetValueOnGameThread();

		if(Value >= 0.0f)
		{
			FinalPostProcessSettings.ScreenSpaceReflectionMaxRoughness = Value;
		}
	}

	{
		static const auto AmbientOcclusionStaticFractionCVar = IConsoleManager::Get().FindTConsoleVariableDataFloat(TEXT("r.AmbientOcclusionStaticFraction"));

		float Value = AmbientOcclusionStaticFractionCVar->GetValueOnGameThread();

		if(Value >= 0.0)
		{
			FinalPostProcessSettings.AmbientOcclusionStaticFraction = Value;
		}
	}

	if(!Family->EngineShowFlags.AmbientOcclusion || !Family->EngineShowFlags.ScreenSpaceAO)
	{
		FinalPostProcessSettings.AmbientOcclusionIntensity = 0;
	}

	{
		static const auto AmbientOcclusionRadiusScaleCVar = IConsoleManager::Get().FindTConsoleVariableDataFloat(TEXT("r.AmbientOcclusionRadiusScale"));

		float Scale = FMath::Clamp(AmbientOcclusionRadiusScaleCVar->GetValueOnGameThread(), 0.1f, 15.0f);

		FinalPostProcessSettings.AmbientOcclusionRadius *= Scale;
	}

	{
		float Scale = FMath::Clamp(CVarSSAOFadeRadiusScale.GetValueOnGameThread(), 0.01f, 50.0f);

		FinalPostProcessSettings.AmbientOcclusionDistance_DEPRECATED *= Scale;
	}

	{
		float Value = FMath::Clamp(CVarMotionBlurScale.GetValueOnGameThread(), 0.0f, 50.0f);

		FinalPostProcessSettings.MotionBlurAmount *= Value;
	}

	{
		float Value = CVarMotionBlurAmount.GetValueOnGameThread();

		if(Value >= 0.0f)
		{
			FinalPostProcessSettings.MotionBlurAmount = Value;
		}
	}

	{
		float Value = CVarMotionBlurMax.GetValueOnGameThread();

		if(Value >= 0.0f)
		{
			FinalPostProcessSettings.MotionBlurMax = Value;
		}
	}

	{
		int32 TargetFPS = CVarMotionBlurTargetFPS.GetValueOnGameThread();

		if (TargetFPS >= 0)
		{
			FinalPostProcessSettings.MotionBlurTargetFPS = TargetFPS;
		}
	}

	{
		float Value = CVarSceneColorFringeMax.GetValueOnGameThread();

		if (Value >= 0.0f)
		{
			FinalPostProcessSettings.SceneFringeIntensity = FMath::Min(FinalPostProcessSettings.SceneFringeIntensity, Value);
		}
		else if (Value == -2.0f)
		{
			FinalPostProcessSettings.SceneFringeIntensity = 5.0f;
		}

		if(!Family->EngineShowFlags.SceneColorFringe || !Family->EngineShowFlags.CameraImperfections)
		{
			FinalPostProcessSettings.SceneFringeIntensity = 0;
		}
	}

	if (!Family->EngineShowFlags.Lighting || !Family->EngineShowFlags.GlobalIllumination)
	{
		FinalPostProcessSettings.IndirectLightingColor = FLinearColor(0,0,0,0);
		FinalPostProcessSettings.IndirectLightingIntensity = 0.0f;
	}

	if (AllowDebugViewmodes())
	{
		ConfigureBufferVisualizationSettings();
	}

#if !(UE_BUILD_SHIPPING)
	if (Family->EngineShowFlags.IsVisualizeCalibrationEnabled())
	{
		ConfigureVisualizeCalibrationSettings();
	}
#endif

#if WITH_EDITOR
	FHighResScreenshotConfig& Config = GetHighResScreenshotConfig();

	// Pass highres screenshot materials through post process settings
	FinalPostProcessSettings.HighResScreenshotMaterial = Config.HighResScreenshotMaterial;
	FinalPostProcessSettings.HighResScreenshotMaskMaterial = Config.HighResScreenshotMaskMaterial;
	FinalPostProcessSettings.HighResScreenshotCaptureRegionMaterial = NULL;

	// If the highres screenshot UI is open and we're not taking a highres screenshot this frame
	if (Config.bDisplayCaptureRegion && !GIsHighResScreenshot)
	{
		// Only enable the capture region effect if the capture region is different from the view rectangle...
		if ((Config.UnscaledCaptureRegion != UnscaledViewRect) && (Config.UnscaledCaptureRegion.Area() > 0) && (State != NULL))
		{
			// ...and if this is the viewport associated with the highres screenshot UI
			auto ConfigViewport = Config.TargetViewport.Pin();
			if (ConfigViewport.IsValid() && Family->RenderTarget == ConfigViewport.Get())
			{
				static const FName ParamName = "RegionRect";
				FLinearColor NormalizedCaptureRegion;

				// Normalize capture region into view rectangle
				NormalizedCaptureRegion.R = (float)Config.UnscaledCaptureRegion.Min.X / (float)UnscaledViewRect.Width();
				NormalizedCaptureRegion.G = (float)Config.UnscaledCaptureRegion.Min.Y / (float)UnscaledViewRect.Height();
				NormalizedCaptureRegion.B = (float)Config.UnscaledCaptureRegion.Max.X / (float)UnscaledViewRect.Width();
				NormalizedCaptureRegion.A = (float)Config.UnscaledCaptureRegion.Max.Y / (float)UnscaledViewRect.Height();

				// Get a MID for drawing this frame and push the capture region into the shader parameter
				FinalPostProcessSettings.HighResScreenshotCaptureRegionMaterial = State->GetReusableMID(Config.HighResScreenshotCaptureRegionMaterial);
				FinalPostProcessSettings.HighResScreenshotCaptureRegionMaterial->SetVectorParameterValue(ParamName, NormalizedCaptureRegion);
			}
		}
	}
#endif // WITH_EDITOR

	if (State != nullptr)
	{
		State->SetExposureCompensationCurve(FinalPostProcessSettings.AutoExposureBiasCurve);
	}

#if DO_CHECK || USING_CODE_ANALYSIS
	check(VerifyMembersChecks());
#endif
}

void FSceneView::ConfigureBufferVisualizationSettings()
{
	bool bBufferDumpingRequired = (FScreenshotRequest::IsScreenshotRequested() || GIsHighResScreenshot || GIsDumpingMovie);
	bool bVisualizationRequired = Family->EngineShowFlags.VisualizeBuffer;

	if (bVisualizationRequired || bBufferDumpingRequired)
	{
		FinalPostProcessSettings.bBufferVisualizationDumpRequired = bBufferDumpingRequired;
		FinalPostProcessSettings.BufferVisualizationOverviewMaterials.Empty();

		if (bBufferDumpingRequired)
		{
			FinalPostProcessSettings.BufferVisualizationDumpBaseFilename = FPaths::GetBaseFilename(FScreenshotRequest::GetFilename(), false);
		}

		// Get the list of requested buffers from the console
		static IConsoleVariable* CVar = IConsoleManager::Get().FindConsoleVariable(TEXT("r.BufferVisualizationOverviewTargets"));
		FString SelectedMaterialNames = CVar->GetString();

		FBufferVisualizationData& BufferVisualizationData = GetBufferVisualizationData();

		if (BufferVisualizationData.IsDifferentToCurrentOverviewMaterialNames(SelectedMaterialNames))
		{
			FString Left, Right;

			// Update our record of the list of materials we've been asked to display
			BufferVisualizationData.SetCurrentOverviewMaterialNames(SelectedMaterialNames);
			BufferVisualizationData.GetOverviewMaterials().Empty();

			// Extract each material name from the comma separated string
			while (SelectedMaterialNames.Len())
			{
				// Detect last entry in the list
				if (!SelectedMaterialNames.Split(TEXT(","), &Left, &Right))
				{
					Left = SelectedMaterialNames;
					Right = FString();
				}

				// Lookup this material from the list that was parsed out of the global ini file
				Left.TrimStartInline();
				UMaterialInterface* Material = BufferVisualizationData.GetMaterial(*Left);

				if (Material == NULL && Left.Len() > 0)
				{
					UE_LOGF(LogBufferVisualization, Warning, "Unknown material '%ls'", *Left);
				}

				// Add this material into the material list in the post processing settings so that the render thread
				// can pick them up and draw them into the on-screen tiles
				BufferVisualizationData.GetOverviewMaterials().Add(Material);

				SelectedMaterialNames = Right;
			}
		}

		// Copy current material list into settings material list
		for (TArray<UMaterialInterface*>::TConstIterator It = BufferVisualizationData.GetOverviewMaterials().CreateConstIterator(); It; ++It)
		{
			UMaterialInterface* VisMat = *It;
			if(VisMat && bBufferDumpingRequired)
			{
				VisMat->EnsureIsComplete();
			}
			FinalPostProcessSettings.BufferVisualizationOverviewMaterials.Add(VisMat);
		}
	}
}

#if !(UE_BUILD_SHIPPING)

void FSceneView::ConfigureVisualizeCalibrationSettings()
{
	const URendererSettings* Settings = GetDefault<URendererSettings>();
	check(Settings);
	
	auto ConfigureCalibrationSettings = [](const FSoftObjectPath& InPath, UMaterialInterface*& OutMaterialInterface, FName& OutMaterialName)
	{
		if (InPath.IsValid())
		{
			if (UMaterial* Material = Cast<UMaterial>(InPath.TryLoad()))
			{
				OutMaterialInterface = Material;
				OutMaterialName = *Material->GetPathName();
			}
			else
			{
				UE_LOGF(LogBufferVisualization, Warning, "Error loading material '%ls'", *InPath.ToString());
				OutMaterialInterface = nullptr;
				OutMaterialName = NAME_None;
			}
		}
	};

	if (Family->EngineShowFlags.VisualizeCalibrationColor)
	{
		ConfigureCalibrationSettings(Settings->VisualizeCalibrationColorMaterialPath, FinalPostProcessSettings.VisualizeCalibrationColorMaterial, CurrentVisualizeCalibrationColorMaterialName);
	}
	else if (Family->EngineShowFlags.VisualizeCalibrationGrayscale)
	{
		ConfigureCalibrationSettings(Settings->VisualizeCalibrationGrayscaleMaterialPath, FinalPostProcessSettings.VisualizeCalibrationGrayscaleMaterial, CurrentVisualizeCalibrationGrayscaleMaterialName);
	}
	else if (Family->EngineShowFlags.VisualizeCalibrationCustom)
	{
		ConfigureCalibrationSettings(Settings->VisualizeCalibrationCustomMaterialPath, FinalPostProcessSettings.VisualizeCalibrationCustomMaterial, CurrentVisualizeCalibrationCustomMaterialName);
	}
}

#endif // #if !(UE_BUILD_SHIPPING)

EShaderPlatform FSceneView::GetShaderPlatform() const
{
	return GShaderPlatformForFeatureLevel[GetFeatureLevel()];
}

bool FSceneView::IsInstancedStereoPass() const
{
	return IsInstancedStereo() && IStereoRendering::IsAPrimaryView(*this);
}

FVector4f FSceneView::GetScreenPositionScaleBias(const FIntPoint& BufferSize, const FIntRect& ViewRect)
{
	const float InvBufferSizeX = 1.0f / BufferSize.X;
	const float InvBufferSizeY = 1.0f / BufferSize.Y;

	// to bring NDC (-1..1, 1..-1) into 0..1 UV for BufferSize textures
	return FVector4f(
		ViewRect.Width() * InvBufferSizeX / +2.0f,
		ViewRect.Height() * InvBufferSizeY / (-2.0f * GProjectionSignY),
		// Warning: note legacy flipped Y and X
		(ViewRect.Height() / 2.0f + ViewRect.Min.Y) * InvBufferSizeY,
		(ViewRect.Width() / 2.0f + ViewRect.Min.X) * InvBufferSizeX
		);
}

void SetupViewRectUniformBufferParameters(
	FViewUniformShaderParameters& ViewUniformShaderParameters,
	const FIntPoint& BufferSize,
	const FIntRect& EffectiveViewRect,
	const FViewMatrices& InViewMatrices,
	const FViewMatrices& InPrevViewMatrices,
	const FSetupViewUniformParametersInputs& Inputs)
{
	checkfSlow(EffectiveViewRect.Area() > 0, TEXT("Invalid-size EffectiveViewRect passed to CreateUniformBufferParameters [%d * %d]."), EffectiveViewRect.Width(), EffectiveViewRect.Height());
	ensureMsgf((BufferSize.X > 0 && BufferSize.Y > 0), TEXT("Invalid-size BufferSize passed to CreateUniformBufferParameters [%d * %d]."), BufferSize.X, BufferSize.Y);

	ViewUniformShaderParameters.ViewRectMin = FVector4f(EffectiveViewRect.Min.X, EffectiveViewRect.Min.Y, 0.0f, 0.0f);
	ViewUniformShaderParameters.ViewSizeAndInvSize = FVector4f(EffectiveViewRect.Width(), EffectiveViewRect.Height(), 1.0f / float(EffectiveViewRect.Width()), 1.0f / float(EffectiveViewRect.Height()));
	ViewUniformShaderParameters.ViewRectMinAndSize = FUintVector4(EffectiveViewRect.Min.X, EffectiveViewRect.Min.Y, EffectiveViewRect.Width(), EffectiveViewRect.Height());

	// The light probe ratio is only different during separate forward translucency when r.SeparateTranslucencyScreenPercentage != 100
	ViewUniformShaderParameters.LightProbeSizeRatioAndInvSizeRatio = FVector4f(1.0f, 1.0f, 1.0f, 1.0f);

	// Calculate the vector used by shaders to convert clip space coordinates to texture space.
	const float InvBufferSizeX = 1.0f / BufferSize.X;
	const float InvBufferSizeY = 1.0f / BufferSize.Y;

	ViewUniformShaderParameters.ScreenPositionScaleBias = FSceneView::GetScreenPositionScaleBias(BufferSize, EffectiveViewRect);

	ViewUniformShaderParameters.BufferSizeAndInvSize = FVector4f(BufferSize.X, BufferSize.Y, InvBufferSizeX, InvBufferSizeY);
	ViewUniformShaderParameters.BufferBilinearUVMinMax = FVector4f(
		InvBufferSizeX * (EffectiveViewRect.Min.X + 0.5),
		InvBufferSizeY * (EffectiveViewRect.Min.Y + 0.5),
		InvBufferSizeX * (EffectiveViewRect.Max.X - 0.5),
		InvBufferSizeY * (EffectiveViewRect.Max.Y - 0.5));

	/* Texture Level-of-Detail Strategies for Real-Time Ray Tracing https://developer.nvidia.com/raytracinggems Equation 20 */
	if(Inputs.FOV != 0)
	{
		float RadFOV = (UE_PI / 180.0f) * Inputs.FOV;
		ViewUniformShaderParameters.EyeToPixelSpreadAngle = FPlatformMath::Atan((2.0f * FPlatformMath::Tan(RadFOV * 0.5f)) / BufferSize.Y);
	}
	else
	{
		ViewUniformShaderParameters.EyeToPixelSpreadAngle = 0;
	}

	ViewUniformShaderParameters.MotionBlurNormalizedToPixel = Inputs.MotionBlurMax * EffectiveViewRect.Width() / 100.0f;

	if (Inputs.bNullifyWorldSpacePosition)
	{
		// Forces world space position to 0 and view vector to up.
		ViewUniformShaderParameters.SVPositionToTranslatedWorld = FMatrix44f(
			FMatrix(FPlane(0, 0, 0, 0),
				FPlane(0, 0, 0, 0),
				FPlane(0, 0, 0, 0),
				FPlane(0, 0, 1, 1)));
	}
	else
	{
		// setup a matrix to transform float4(SvPosition.xyz,1) directly to TranslatedWorld (quality, performance as we don't need to convert or use interpolator)

		//	new_xy = (xy - ViewRectMin.xy) * ViewSizeAndInvSize.zw * float2(2,-2) + float2(-1, 1);

		//  transformed into one MAD:  new_xy = xy * ViewSizeAndInvSize.zw * float2(2,-2)      +       (-ViewRectMin.xy) * ViewSizeAndInvSize.zw * float2(2,-2) + float2(-1, 1);

		float Mx = 2.0f * ViewUniformShaderParameters.ViewSizeAndInvSize.Z;
		float My = -2.0f * ViewUniformShaderParameters.ViewSizeAndInvSize.W;
		float Ax = -1.0f - 2.0f * EffectiveViewRect.Min.X * ViewUniformShaderParameters.ViewSizeAndInvSize.Z;
		float Ay = 1.0f + 2.0f * EffectiveViewRect.Min.Y * ViewUniformShaderParameters.ViewSizeAndInvSize.W;

		// http://stackoverflow.com/questions/9010546/java-transformation-matrix-operations

		ViewUniformShaderParameters.SVPositionToTranslatedWorld = FMatrix44f(
			FMatrix(FPlane(Mx, 0, 0, 0),
				FPlane(0, My, 0, 0),
				FPlane(0, 0, 1, 0),
				FPlane(Ax, Ay, 0, 1)) * InViewMatrices.GetClipToTranslatedWorld());
	}
	
	{
		/**
		* Ortho projection does not use FOV calculations, so rather than sourcing the projection matrix values in CommonViewUniformBuffer.ush,
		* the appropriate values are uploaded in the per view uniform buffer (projection matrix values for perspective, 1.0f for ortho).
		* Doing this here avoids unnecessarily checking for perspective vs ortho in shaders at runtime.
		*/
		FVector4f TanAndInvTanFOV = InViewMatrices.GetTanAndInvTanHalfFOV();
		ViewUniformShaderParameters.TanAndInvTanHalfFOV = TanAndInvTanFOV;
		ViewUniformShaderParameters.PrevTanAndInvTanHalfFOV = InPrevViewMatrices.GetTanAndInvTanHalfFOV();
		ViewUniformShaderParameters.FirstPersonTanAndInvTanHalfFOV = InViewMatrices.GetFirstPersonTanAndInvTanHalfFOV();
		ViewUniformShaderParameters.PrevFirstPersonTanAndInvTanHalfFOV = InPrevViewMatrices.GetFirstPersonTanAndInvTanHalfFOV();

		if (InViewMatrices.IsPerspectiveProjection())
		{
			ViewUniformShaderParameters.WorldDepthToPixelWorldRadius = FVector2f(TanAndInvTanFOV.X / float(EffectiveViewRect.Width()), 0.0f);
			ViewUniformShaderParameters.ScreenRayLengthMultiplier = FVector4f(TanAndInvTanFOV.X, TanAndInvTanFOV.Y, 0, 0);
		}
		else
		{
			ViewUniformShaderParameters.WorldDepthToPixelWorldRadius = FVector2f(0.0f, TanAndInvTanFOV.X);
			ViewUniformShaderParameters.ScreenRayLengthMultiplier = FVector4f(0,0,TanAndInvTanFOV.X, TanAndInvTanFOV.Y);
		}
	}
	
	float FovFixX = ViewUniformShaderParameters.TanAndInvTanHalfFOV.X;
	float FovFixY = ViewUniformShaderParameters.TanAndInvTanHalfFOV.Y;

	ViewUniformShaderParameters.ScreenToViewSpace.X = BufferSize.X * ViewUniformShaderParameters.ViewSizeAndInvSize.Z * 2 * FovFixX;
	ViewUniformShaderParameters.ScreenToViewSpace.Y = BufferSize.Y * ViewUniformShaderParameters.ViewSizeAndInvSize.W  * -2 * FovFixY;

	ViewUniformShaderParameters.ScreenToViewSpace.Z = -((ViewUniformShaderParameters.ViewRectMin.X * ViewUniformShaderParameters.ViewSizeAndInvSize.Z * 2 * FovFixX) + FovFixX);
	ViewUniformShaderParameters.ScreenToViewSpace.W = (ViewUniformShaderParameters.ViewRectMin.Y * ViewUniformShaderParameters.ViewSizeAndInvSize.W * 2 * FovFixY) + FovFixY;

	ViewUniformShaderParameters.ViewResolutionFraction = EffectiveViewRect.Width() / (float)Inputs.UnscaledViewRect.Width();
}

void SetupCommonViewUniformBufferParameters(
	FViewUniformShaderParameters& ViewUniformShaderParameters,
	const FIntPoint& BufferSize,
	int32 NumMSAASamples,
	const FIntRect& EffectiveViewRect,
	const FViewMatrices& InViewMatrices,
	const FViewMatrices& InPrevViewMatrices,
	const FSetupViewUniformParametersInputs& Inputs)
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_SetupCommonViewUniformBufferParameters);
	FVector4f LocalDiffuseOverrideParameter = Inputs.DiffuseOverrideParameter;
	FVector2f LocalRoughnessOverrideParameter = Inputs.RoughnessOverrideParameter;

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	{
		// assuming we have no color in the multipliers
		float MinValue = LocalDiffuseOverrideParameter.X;
		float MaxValue = MinValue + LocalDiffuseOverrideParameter.W;

		float NewMinValue = FMath::Max(MinValue, CVarDiffuseColorMin.GetValueOnRenderThread());
		float NewMaxValue = FMath::Min(MaxValue, CVarDiffuseColorMax.GetValueOnRenderThread());

		LocalDiffuseOverrideParameter.X = LocalDiffuseOverrideParameter.Y = LocalDiffuseOverrideParameter.Z = NewMinValue;
		LocalDiffuseOverrideParameter.W = NewMaxValue - NewMinValue;
	}
	{
		float MinValue = LocalRoughnessOverrideParameter.X;
		float MaxValue = MinValue + LocalRoughnessOverrideParameter.Y;

		float NewMinValue = FMath::Max(MinValue, CVarRoughnessMin.GetValueOnRenderThread());
		float NewMaxValue = FMath::Min(MaxValue, CVarRoughnessMax.GetValueOnRenderThread());

		LocalRoughnessOverrideParameter.X = NewMinValue;
		LocalRoughnessOverrideParameter.Y = NewMaxValue - NewMinValue;
	}
#endif

	const FDFRelativeViewMatrices RelativeMatrices = FDFRelativeViewMatrices::Create(InViewMatrices, InPrevViewMatrices);
	const FDFVector3 AbsoluteViewOrigin(InViewMatrices.GetViewOrigin());
	const FVector ViewOriginHigh(AbsoluteViewOrigin.High);
	const FDFVector3 AbsolutePreViewTranslation(InViewMatrices.GetPreViewTranslation()); // Usually equal to -AbsoluteViewOrigin, but there are some ortho edge cases
	
	ViewUniformShaderParameters.NumSceneColorMSAASamples = NumMSAASamples;
	ViewUniformShaderParameters.ViewToTranslatedWorld = FMatrix44f(InViewMatrices.GetOverriddenInvTranslatedViewMatrix());	// LWC_TODO: Precision - Validate all float variant casts here.
	ViewUniformShaderParameters.TranslatedWorldToClip = FMatrix44f(InViewMatrices.GetTranslatedWorldToClip());
	ViewUniformShaderParameters.RelativeWorldToClip = RelativeMatrices.RelativeWorldToClip;
	ViewUniformShaderParameters.ClipToRelativeWorld = RelativeMatrices.ClipToRelativeWorld;
	ViewUniformShaderParameters.TranslatedWorldToView = FMatrix44f(InViewMatrices.GetOverriddenTranslatedViewMatrix());
	ViewUniformShaderParameters.TranslatedWorldToCameraView = FMatrix44f(InViewMatrices.GetTranslatedViewMatrix());
	ViewUniformShaderParameters.CameraViewToTranslatedWorld = FMatrix44f(InViewMatrices.GetInvTranslatedViewMatrix());
	ViewUniformShaderParameters.ViewToClip = RelativeMatrices.ViewToClip;
	ViewUniformShaderParameters.ViewToClipNoAA = FMatrix44f(InViewMatrices.GetViewToClipNoAA());
	ViewUniformShaderParameters.ClipToView = RelativeMatrices.ClipToView;
	ViewUniformShaderParameters.ClipToTranslatedWorld = FMatrix44f(InViewMatrices.GetClipToTranslatedWorld());
	ViewUniformShaderParameters.ViewForward = (FVector3f)InViewMatrices.GetOverriddenTranslatedViewMatrix().GetColumn(2);
	ViewUniformShaderParameters.ViewUp = (FVector3f)InViewMatrices.GetOverriddenTranslatedViewMatrix().GetColumn(1);
	ViewUniformShaderParameters.ViewRight = (FVector3f)InViewMatrices.GetOverriddenTranslatedViewMatrix().GetColumn(0);
	ViewUniformShaderParameters.HMDViewNoRollUp = (FVector3f)InViewMatrices.GetHMDViewMatrixNoRoll().GetColumn(1);
	ViewUniformShaderParameters.HMDViewNoRollRight = (FVector3f)InViewMatrices.GetHMDViewMatrixNoRoll().GetColumn(0);
	ViewUniformShaderParameters.InvDeviceZToWorldZTransform = Inputs.InvDeviceZToWorldZTransform;
	FDFVector4 WorldViewOriginDF { (FVector4f)(InViewMatrices.GetOverriddenInvTranslatedViewMatrix().TransformPosition(FVector(0)) - InViewMatrices.GetPreViewTranslation()) };
	ViewUniformShaderParameters.WorldViewOriginHigh = WorldViewOriginDF.High;
	ViewUniformShaderParameters.WorldViewOriginLow = WorldViewOriginDF.Low;
	ViewUniformShaderParameters.ViewOriginHigh = AbsoluteViewOrigin.High;
	ViewUniformShaderParameters.ViewOriginLow = AbsoluteViewOrigin.Low;
	ViewUniformShaderParameters.TranslatedWorldCameraOrigin = FVector3f(InViewMatrices.GetViewOrigin() + InViewMatrices.GetPreViewTranslation());
	ViewUniformShaderParameters.PreViewTranslationHigh = AbsolutePreViewTranslation.High;
	ViewUniformShaderParameters.PreViewTranslationLow = AbsolutePreViewTranslation.Low;
	ViewUniformShaderParameters.PrevViewToClip = FMatrix44f(InPrevViewMatrices.GetViewToClip());
	ViewUniformShaderParameters.PrevClipToView = RelativeMatrices.PrevClipToView;
	ViewUniformShaderParameters.PrevTranslatedWorldToClip = FMatrix44f(InPrevViewMatrices.GetTranslatedWorldToClip());
	// EffectiveTranslatedViewMatrix != InViewMatrices.TranslatedViewMatrix in the shadow pass
	// and we don't have EffectiveTranslatedViewMatrix for the previous frame to set up PrevTranslatedWorldToView
	// but that is fine to set up PrevTranslatedWorldToView as same as PrevTranslatedWorldToCameraView
	// since the shadow pass doesn't require previous frame computation.
	ViewUniformShaderParameters.PrevTranslatedWorldToView = FMatrix44f(InPrevViewMatrices.GetOverriddenTranslatedViewMatrix());
	ViewUniformShaderParameters.PrevViewToTranslatedWorld = FMatrix44f(InPrevViewMatrices.GetOverriddenInvTranslatedViewMatrix());
	ViewUniformShaderParameters.PrevTranslatedWorldToCameraView = FMatrix44f(InPrevViewMatrices.GetTranslatedViewMatrix());
	ViewUniformShaderParameters.PrevCameraViewToTranslatedWorld = FMatrix44f(InPrevViewMatrices.GetInvTranslatedViewMatrix());
	FDFVector3 PrevWorldCameraOriginDF { InPrevViewMatrices.GetViewOrigin() };
	ViewUniformShaderParameters.PrevWorldCameraOriginHigh = PrevWorldCameraOriginDF.High;
	ViewUniformShaderParameters.PrevWorldCameraOriginLow = PrevWorldCameraOriginDF.Low;
	// previous view world origin is going to be needed only in the base pass or shadow pass
	// therefore is same as previous camera world origin.
	ViewUniformShaderParameters.PrevWorldViewOriginHigh = ViewUniformShaderParameters.PrevWorldCameraOriginHigh;
	ViewUniformShaderParameters.PrevWorldViewOriginLow = ViewUniformShaderParameters.PrevWorldCameraOriginLow;
	ViewUniformShaderParameters.PrevTranslatedWorldCameraOrigin = FVector3f(InPrevViewMatrices.GetViewOrigin() + InPrevViewMatrices.GetPreViewTranslation());
	FDFVector3 PrevPreViewTranslationDF { InPrevViewMatrices.GetPreViewTranslation() };
	ViewUniformShaderParameters.PrevPreViewTranslationHigh = PrevPreViewTranslationDF.High;
	ViewUniformShaderParameters.PrevPreViewTranslationLow = PrevPreViewTranslationDF.Low;
	ViewUniformShaderParameters.PrevClipToRelativeWorld = RelativeMatrices.PrevClipToRelativeWorld;

	// TileOffset variables for materials. Calculating these on-GPU was too expensive for low end platforms.
	const FLargeWorldRenderPosition AbsoluteViewOriginTO(InViewMatrices.GetViewOrigin());
	const FVector ViewTileOffset = AbsoluteViewOriginTO.GetTileOffset();
	ViewUniformShaderParameters.ViewTilePosition = AbsoluteViewOriginTO.GetTile();
	ViewUniformShaderParameters.RelativeWorldCameraOriginTO = FVector3f(InViewMatrices.GetViewOrigin() - ViewTileOffset);
	ViewUniformShaderParameters.RelativeWorldViewOriginTO = (FVector4f)(InViewMatrices.GetOverriddenInvTranslatedViewMatrix().TransformPosition(FVector(0)) - InViewMatrices.GetPreViewTranslation() - ViewTileOffset);
	ViewUniformShaderParameters.RelativePreViewTranslationTO = FVector3f(InViewMatrices.GetPreViewTranslation() + ViewTileOffset);
	ViewUniformShaderParameters.PrevRelativeWorldCameraOriginTO = FVector3f(InPrevViewMatrices.GetViewOrigin() - ViewTileOffset);
	ViewUniformShaderParameters.PrevRelativeWorldViewOriginTO = ViewUniformShaderParameters.PrevRelativeWorldCameraOriginTO;
	ViewUniformShaderParameters.RelativePrevPreViewTranslationTO = FVector3f(InPrevViewMatrices.GetPreViewTranslation() + ViewTileOffset);

	// Convert global clipping plane to translated world space
	const FPlane4f TranslatedGlobalClippingPlane(Inputs.GlobalClippingPlane.TranslateBy(InViewMatrices.GetPreViewTranslation()));

	ViewUniformShaderParameters.GlobalClippingPlane = FVector4f(TranslatedGlobalClippingPlane.X, TranslatedGlobalClippingPlane.Y, TranslatedGlobalClippingPlane.Z, -TranslatedGlobalClippingPlane.W);

	ViewUniformShaderParameters.FieldOfViewWideAngles = FVector2f(2.f * InViewMatrices.ComputeHalfFieldOfViewPerAxis());	// LWC_TODO: Precision loss
	ViewUniformShaderParameters.PrevFieldOfViewWideAngles = FVector2f(2.f * InPrevViewMatrices.ComputeHalfFieldOfViewPerAxis());	// LWC_TODO: Precision loss
	ViewUniformShaderParameters.FirstPersonFieldOfViewWideAngles = FVector2f(2.f * InViewMatrices.GetFirstPersonHalfFieldOfViewPerAxis());
	ViewUniformShaderParameters.PrevFirstPersonFieldOfViewWideAngles = FVector2f(2.f * InPrevViewMatrices.GetFirstPersonHalfFieldOfViewPerAxis());
	ViewUniformShaderParameters.FirstPersonScale = InViewMatrices.GetFirstPersonScale();
	ViewUniformShaderParameters.PrevFirstPersonScale = InPrevViewMatrices.GetFirstPersonScale();
	ViewUniformShaderParameters.DiffuseOverrideParameter = LocalDiffuseOverrideParameter;
	ViewUniformShaderParameters.SpecularOverrideParameter = Inputs.SpecularOverrideParameter;
	ViewUniformShaderParameters.NormalOverrideParameter = Inputs.NormalOverrideParameter;
	ViewUniformShaderParameters.RoughnessOverrideParameter = LocalRoughnessOverrideParameter;
	ViewUniformShaderParameters.AmbientOcclusionOverrideParameter = Inputs.AmbientOcclusionOverrideParameter;
	ViewUniformShaderParameters.WorldCameraMovementSinceLastFrame = FVector3f(InViewMatrices.GetViewOrigin() - InPrevViewMatrices.GetViewOrigin());
	ViewUniformShaderParameters.CullingSign = Inputs.bReverseCulling ? -1.0f : 1.0f;
	ViewUniformShaderParameters.NearPlane = InViewMatrices.ComputeNearPlane();
	ViewUniformShaderParameters.OrthoFarPlane = InViewMatrices.ComputeOrthoFarPlane();
	ViewUniformShaderParameters.MaterialTextureMipBias = 0.0f;
	ViewUniformShaderParameters.MaterialTextureDerivativeMultiply = 1.0f;
	ViewUniformShaderParameters.ResolutionFractionAndInv = FVector2f(1.0f, 1.0f);
	ViewUniformShaderParameters.ProjectionDepthThicknessScale = InViewMatrices.GetPerProjectionDepthThicknessScale();

	ViewUniformShaderParameters.bCheckerboardSubsurfaceProfileRendering = 0;

	ViewUniformShaderParameters.ScreenToRelativeWorld = 
		FMatrix44f(InViewMatrices.GetScreenToClipMatrix()) * ViewUniformShaderParameters.ClipToRelativeWorld;

	ViewUniformShaderParameters.ScreenToTranslatedWorld = 
		FMatrix44f(InViewMatrices.GetScreenToClipMatrix() * InViewMatrices.GetClipToTranslatedWorld());

	ViewUniformShaderParameters.MobileMultiviewShadowTransform = 
		FMatrix44f(InViewMatrices.GetScreenToClipMatrix() * InViewMatrices.GetClipToTranslatedWorld() * FTranslationMatrix(-InViewMatrices.GetPreViewTranslation()));

	ViewUniformShaderParameters.MobileMultiviewDecalTransform =
		FMatrix44f(FTranslationMatrix(InViewMatrices.GetPreViewTranslation()) * InViewMatrices.GetTranslatedWorldToClip());

	ViewUniformShaderParameters.PrevScreenToTranslatedWorld = 
		FMatrix44f(InPrevViewMatrices.GetScreenToClipMatrix() * InPrevViewMatrices.GetClipToTranslatedWorld());

	{
		FVector DeltaTranslation = InPrevViewMatrices.GetPreViewTranslation() - InViewMatrices.GetPreViewTranslation();
		FMatrix InvViewProj = InViewMatrices.ComputeInvProjectionNoAAMatrix() * InViewMatrices.GetTranslatedViewMatrix().GetTransposed();
		FMatrix PrevViewProj = FTranslationMatrix(DeltaTranslation) * InPrevViewMatrices.GetTranslatedViewMatrix() * InPrevViewMatrices.ComputeProjectionNoAAMatrix();

		ViewUniformShaderParameters.ClipToPrevClip = FMatrix44f(InvViewProj * PrevViewProj);		// LWC_TODO: Precision loss?
	}

	{
		FVector DeltaTranslation = InPrevViewMatrices.GetPreViewTranslation() - InViewMatrices.GetPreViewTranslation();
		FMatrix InvViewProj = InViewMatrices.GetClipToView() * InViewMatrices.GetTranslatedViewMatrix().GetTransposed();
		FMatrix PrevViewProj = FTranslationMatrix(DeltaTranslation) * InPrevViewMatrices.GetTranslatedViewMatrix() * InPrevViewMatrices.GetViewToClip();

		ViewUniformShaderParameters.ClipToPrevClipWithAA = FMatrix44f(InvViewProj * PrevViewProj);		// LWC_TODO: Precision loss?
	}

	ViewUniformShaderParameters.FirstPersonTransform = FMatrix44f(InViewMatrices.GetWorldToFirstPerson());
	ViewUniformShaderParameters.PrevFirstPersonTransform = FMatrix44f(InPrevViewMatrices.GetWorldToFirstPerson());
	ViewUniformShaderParameters.InvFirstPersonTransform = FMatrix44f(InViewMatrices.GetFirstPersonToWorld());

	// LWC_TODO: precision loss? These values are probably quite small and easily within float range.
	ViewUniformShaderParameters.TemporalAAJitter = FVector4f(
		(float)InViewMatrices.GetTemporalAAJitter().X, (float)InViewMatrices.GetTemporalAAJitter().Y,
		(float)InPrevViewMatrices.GetTemporalAAJitter().X, (float)InPrevViewMatrices.GetTemporalAAJitter().Y );

	ViewUniformShaderParameters.DebugViewModeMask = Inputs.DebugViewShaderMode != DVSM_None ? 1 : 0;
	ViewUniformShaderParameters.UnlitViewmodeMask = Inputs.EngineShowFlags && ((!Inputs.EngineShowFlags->Lighting && Inputs.EngineShowFlags->UnlitViewmode) || Inputs.EngineShowFlags->PathTracing) ? 1 : 0;
	ViewUniformShaderParameters.OutOfBoundsMask = Inputs.EngineShowFlags && Inputs.EngineShowFlags->VisualizeOutOfBoundsPixels ? 1 : 0;

#if !(UE_BUILD_SHIPPING)
	float OverrideTimeMaterialExpression = CVarOverrideTimeMaterialExpressions.GetValueOnRenderThread();
	if (OverrideTimeMaterialExpression >= 0.0f)
	{
		ViewUniformShaderParameters.PrevFrameGameTime = OverrideTimeMaterialExpression;
		ViewUniformShaderParameters.PrevFrameRealTime = OverrideTimeMaterialExpression;
		ViewUniformShaderParameters.GameTime = OverrideTimeMaterialExpression;
		ViewUniformShaderParameters.RealTime = OverrideTimeMaterialExpression;
		ViewUniformShaderParameters.DeltaTime = 0.0f;
	}
	else
#endif
	{
		ViewUniformShaderParameters.PrevFrameGameTime = Inputs.Time.GetWorldTimeSeconds() - Inputs.Time.GetDeltaWorldTimeSeconds();
		ViewUniformShaderParameters.PrevFrameRealTime = Inputs.Time.GetRealTimeSeconds() - Inputs.Time.GetDeltaRealTimeSeconds();
		ViewUniformShaderParameters.GameTime = Inputs.Time.GetWorldTimeSeconds();
		ViewUniformShaderParameters.RealTime = Inputs.Time.GetRealTimeSeconds();
		ViewUniformShaderParameters.DeltaTime = Inputs.Time.GetDeltaWorldTimeSeconds();
	}

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	static FIntPoint LockedCursorPos = Inputs.CursorPosition;
	if (CVarFreezeMouseCursor.GetValueOnRenderThread() == 0 && Inputs.CursorPosition.X >= 0 && Inputs.CursorPosition.Y >= 0)
	{
		LockedCursorPos = Inputs.CursorPosition;
	}

	if (CVarMouseCursorX.GetValueOnRenderThread() >= 0 && CVarMouseCursorY.GetValueOnRenderThread() >= 0)
	{
		LockedCursorPos.X = CVarMouseCursorX.GetValueOnRenderThread();
		LockedCursorPos.Y = CVarMouseCursorY.GetValueOnRenderThread();
	}

	ViewUniformShaderParameters.CursorPosition = LockedCursorPos;
#endif

	ViewUniformShaderParameters.Random = FMath::Rand();
	// FrameNumber corresponds to how many times FRendererModule::BeginRenderingViewFamilies has been called, so multi views of the same frame have incremental values.
	ViewUniformShaderParameters.FrameNumber = Inputs.FrameNumber;
	// FrameCounter is incremented once per engine tick, so multi views of the same frame have the same value.
	ViewUniformShaderParameters.FrameCounter = Inputs.FrameCounter;
	ViewUniformShaderParameters.WorldIsPaused = Inputs.bWorldIsPaused;
	//Set bCameraCut if we switch projection type to ensure histories are updated.
	ViewUniformShaderParameters.CameraCut = Inputs.bCameraCut ? 1 : (InViewMatrices.IsPerspectiveProjection() != InPrevViewMatrices.IsPerspectiveProjection());

	ViewUniformShaderParameters.MinRoughness = FMath::Clamp(CVarGlobalMinRoughnessOverride.GetValueOnRenderThread(), 0.02f, 1.0f);

	//to tail call keep the order and number of parameters of the caller function
	SetupViewRectUniformBufferParameters(ViewUniformShaderParameters, BufferSize, EffectiveViewRect, InViewMatrices, InPrevViewMatrices, Inputs);
}

void FSceneView::SetupViewRectUniformBufferParameters(
	FViewUniformShaderParameters& ViewUniformShaderParameters,
	const FIntPoint& InBufferSize,
	const FIntRect& InEffectiveViewRect,
	const FViewMatrices& InViewMatrices,
	const FViewMatrices& InPrevViewMatrices) const
{
	::SetupViewRectUniformBufferParameters(ViewUniformShaderParameters, InBufferSize, InEffectiveViewRect, InViewMatrices, InPrevViewMatrices, FSetupViewUniformParametersInputs::Create(*this));
}

void FSceneView::SetupCommonViewUniformBufferParameters(
	FViewUniformShaderParameters& ViewUniformShaderParameters,
	const FIntPoint& BufferSize,
	int32 NumMSAASamples,
	const FIntRect& EffectiveViewRect,
	const FViewMatrices& InViewMatrices,
	const FViewMatrices& InPrevViewMatrices) const
{
	::SetupCommonViewUniformBufferParameters(ViewUniformShaderParameters, BufferSize, NumMSAASamples, EffectiveViewRect, InViewMatrices, InPrevViewMatrices, FSetupViewUniformParametersInputs::Create(*this));
}

bool FSceneView::HasValidEyeAdaptationBuffer() const
{
	if (EyeAdaptationViewState)
	{
		return EyeAdaptationViewState->HasValidEyeAdaptationBuffer();
	}
	return false;
}

FRDGPooledBuffer* FSceneView::GetEyeAdaptationBuffer() const
{
	if (EyeAdaptationViewState)
	{
		return EyeAdaptationViewState->GetCurrentEyeAdaptationBuffer();
	}
	return nullptr;
}

/** Returns the eye adaptation exposure or 0.0f if it doesn't exist. */
float FSceneView::GetLastEyeAdaptationExposure() const
{
	if (EyeAdaptationViewState)
	{
		return EyeAdaptationViewState->GetLastEyeAdaptationExposure();
	}
	return 0.0f;
}

/** Returns the eye adaptation luminance or 0.0f if it doesn't exist. */
float FSceneView::GetLastAverageSceneLuminance() const
{
	if (EyeAdaptationViewState)
	{
		return EyeAdaptationViewState->GetLastAverageSceneLuminance();
	}
	return 0.0f;
}

const FSceneView* FSceneView::GetPrimarySceneView() const
{
	// It is valid for this function to return itself if it's already the primary view.
	if (Family && Family->Views.IsValidIndex(PrimaryViewIndex))
	{
		return Family->Views[PrimaryViewIndex];
	}
	return this;
}

const FSceneView* FSceneView::GetInstancedSceneView() const
{
	// if we don't have ISR or MMV enabled, or we're rendering a full pass, we don't have instanced views
	if (bIsSinglePassStereo && StereoPass != EStereoscopicPass::eSSP_FULL)
	{
		// If called on the first secondary view it'll return itself.
		if (Family && Family->Views.IsValidIndex(PrimaryViewIndex + 1))
		{
			const FSceneView* SecondaryView = Family->Views[PrimaryViewIndex + 1];
			return IStereoRendering::IsASecondaryView(*SecondaryView) ? SecondaryView : nullptr;
		}
	}
	return nullptr;
}

TArray<const FSceneView*> FSceneView::GetSecondaryViews() const
{
	// If called on a secondary view we'll return all other secondary views, including this view.
	TArray<const FSceneView*> Views;
	Views.Reserve(Family->Views.Num());
	for (int32 ViewIndex = PrimaryViewIndex + 1; ViewIndex < Family->Views.Num() &&
		IStereoRendering::IsASecondaryView(*Family->Views[ViewIndex]); ViewIndex++)
	{
		Views.Add(Family->Views[ViewIndex]);
	}
	return Views;
}

FSceneViewFamily::ConstructionValues::ConstructionValues(
	const FRenderTarget* InRenderTarget,
	FSceneInterface* InScene,
	const FEngineShowFlags& InEngineShowFlags
	)
:	RenderTarget(InRenderTarget)
,	RenderTargetDepth(nullptr)
,	Scene(InScene)
,	EngineShowFlags(InEngineShowFlags)
,	ViewModeParam(-1)
,	bAdditionalViewFamily(false)
,	bRealtimeUpdate(false)
,	bDeferClear(false)
,	bResolveScene(true)			
,	bTimesSet(false)
,	bRequireMultiView(false)
{
	if( InScene != NULL )			
	{
		UWorld* World = InScene->GetWorld();
		// Ensure the world is valid and that we are being called from a game thread (GetRealTimeSeconds requires this)
		if( World && IsInGameThread() )
		{	
			SetTime(World->GetTime());
		}
	}
}

FSceneViewFamily::FSceneViewFamily(const ConstructionValues& CVS)
	:
	ViewMode(VMI_Lit),
	RenderTarget(CVS.RenderTarget),
	RenderTargetDepth(CVS.RenderTargetDepth),
	Scene(CVS.Scene),
	EngineShowFlags(CVS.EngineShowFlags),
	Time(CVS.Time),
	FrameNumber(UINT_MAX),
	bAdditionalViewFamily(CVS.bAdditionalViewFamily),
	bRealtimeUpdate(CVS.bRealtimeUpdate),
	bDeferClear(CVS.bDeferClear),
	bResolveScene(CVS.bResolveScene),
	bMultiGPUForkAndJoin(false),
	SceneCaptureSource(SCS_FinalColorLDR),
	SceneCaptureCompositeMode(SCCM_Overwrite),
	bWorldIsPaused(false),
	bIsHDR(false),
	bRequireMultiView(CVS.bRequireMultiView),
	SecondaryViewFraction(1.0f),
	SecondaryScreenPercentageMethod(ESecondaryScreenPercentageMethod::LowerPixelDensitySimulation),
	SceneRenderer(nullptr),
	ScreenPercentageInterface(nullptr),
	TemporalUpscalerInterface(nullptr),
	MegaFiltersInterface(nullptr),
	PrimarySpatialUpscalerInterface(nullptr),
	SecondarySpatialUpscalerInterface(nullptr)
{
	// If we do not pass a valid scene pointer then SetTime() must be called to initialized with valid times.
	ensure(CVS.bTimesSet);

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	int32 Value = CVarRenderTimeFrozen.GetValueOnAnyThread();
	if(Value)
	{
		Time = FGameTime::CreateDilated(0.0, Time.GetDeltaRealTimeSeconds(), 0.0, Time.GetDeltaWorldTimeSeconds());
	}
#endif

#if WITH_DEBUG_VIEW_MODES
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	ViewModeParam = CVS.ViewModeParam;
	ViewModeParamName = CVS.ViewModeParamName;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
#endif

	bCurrentlyBeingEdited = false;

	bOverrideVirtualTextureThrottle = false;
	VirtualTextureFeedbackFactor = VirtualTextureScalability::GetVirtualTextureFeedbackFactor();

	if (IsDefaultBackBufferLinearSDR())
	{
		// We preserve linear color values when users select the Float RGBA frame buffer pixel format.
		// Note: This could be removed if our tonemap pass correctly handled DisplayGamma 1.0 (instead of asymmetrically inverting gamma 2.2 before applying sRGB).
		SceneCaptureSource = SCS_FinalToneCurveHDR;
	}

#if !WITH_EDITOR
	check(!EngineShowFlags.StationaryLightOverlap);
#else

	// instead of checking IsGameWorld on rendering thread to see if we allow this flag to be disabled
	// we force it on in the game thread.
	if(IsInGameThread() && Scene)
	{
		UWorld* World = Scene->GetWorld();

		if (World)
		{
			if (World->IsGameWorld())
			{
				EngineShowFlags.LOD = 1;
			}

			// If a single frame step or toggling between Play-in-Editor and Simulate-in-Editor happened this frame, then unpause for this frame.
			bWorldIsPaused = !(World->IsCameraMoveable() || World->bDebugFrameStepExecutedThisFrame || World->bToggledBetweenPIEandSIEThisFrame);
		}
	}

	bDrawBaseInfo = true;
	bNullifyWorldSpacePosition = false;
#endif
	LandscapeLODOverride = -1;

	EngineShowFlags.ScreenPercentage = SupportsScreenPercentage();
}

FSceneViewFamily::~FSceneViewFamily()
{
	// If a screen percentage was given for the view family, delete it since any new copy of a view family will Fork it.
	if (ScreenPercentageInterface)
	{
		delete ScreenPercentageInterface;
	}

	if (TemporalUpscalerInterface)
	{
		// ITemporalUpscaler* is only defined in renderer's private header because no backward compatibility is provided between major version.
		delete reinterpret_cast<ISceneViewFamilyExtention*>(TemporalUpscalerInterface);
	}

	if (MegaFiltersInterface)
	{
		// IMegaFilters* is only defined in renderer's private header because no backward compatibility is provided between major version.
		delete reinterpret_cast<ISceneViewFamilyExtention*>(MegaFiltersInterface);
	}

	if (PrimarySpatialUpscalerInterface)
	{
		// ISpatialUpscaler* is only defined in renderer's private header because no backward compatibility is provided between major version.
		delete reinterpret_cast<ISceneViewFamilyExtention*>(PrimarySpatialUpscalerInterface);
	}

	if (SecondarySpatialUpscalerInterface)
	{
		// ISpatialUpscaler* is only defined in renderer's private header because no backward compatibility is provided between major version.
		delete reinterpret_cast<ISceneViewFamilyExtention*>(SecondarySpatialUpscalerInterface);
	}
}

FSceneViewFamily::FSceneViewFamily(FSceneViewFamily&& InViewFamily)
	: FSceneViewFamily(static_cast<const FSceneViewFamily&>(InViewFamily))
{
	check(ScreenPercentageInterface == nullptr);
	check(TemporalUpscalerInterface == nullptr);
	check(MegaFiltersInterface == nullptr);
	check(PrimarySpatialUpscalerInterface == nullptr);
	check(SecondarySpatialUpscalerInterface == nullptr);
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
FSceneViewFamily::FSceneViewFamily(const FSceneViewFamily&) = default;
PRAGMA_ENABLE_DEPRECATION_WARNINGS

ERHIFeatureLevel::Type FSceneViewFamily::GetFeatureLevel() const
{
	if (Scene)
	{
		return Scene->GetFeatureLevel();
	}
	else
	{
		return GMaxRHIFeatureLevel;
	}
}

bool FSceneViewFamily::SupportsScreenPercentage() const
{
	if (Views.Num() && Views[0]->PrimaryScreenPercentageMethod == EPrimaryScreenPercentageMethod::RawOutput)
	{
		return true;
	}

	if (Scene != nullptr)
	{
		if (GetFeatureLevelShadingPath(Scene->GetFeatureLevel()) == EShadingPath::Mobile)
		{
			const EShaderPlatform ShaderPlatform = Scene->GetShaderPlatform();
			const UE::StereoRenderUtils::FStereoShaderAspects Aspects(ShaderPlatform);

			// Normal post-processing is disabled when using Mobile LDR or the mobile tonemap subpass
			return IsMobileHDR() && !IsMobileTonemapSubpassEnabled(ShaderPlatform, Aspects.IsMobileMultiViewEnabled());
		}

		return true;
	}

	return false;
}

FSceneViewFamilyContext::FSceneViewFamilyContext(const ConstructionValues& CVS)
	: FSceneViewFamily(CVS)
{
}

FSceneViewFamilyContext::~FSceneViewFamilyContext()
{
	// Cleanup the views allocated for this view family.
	for(int32 ViewIndex = 0;ViewIndex < Views.Num();ViewIndex++)
	{
		delete Views[ViewIndex];
	}
}

#if WITH_DEBUG_VIEW_MODES

EDebugViewShaderMode FSceneViewFamily::ChooseDebugViewShaderMode() const
{
	if (EngineShowFlags.ShaderComplexity)
	{
		if (EngineShowFlags.QuadOverdraw)
		{
			return DVSM_QuadComplexity;
		}
		else if (EngineShowFlags.ShaderComplexityWithQuadOverdraw)
		{
			return DVSM_ShaderComplexityContainedQuadOverhead;
		}
		else
		{
			return DVSM_ShaderComplexity;
		}
	}
	else if (EngineShowFlags.PrimitiveDistanceAccuracy)
	{
		return DVSM_PrimitiveDistanceAccuracy;
	}
	else if (EngineShowFlags.MeshUVDensityAccuracy)
	{
		return DVSM_MeshUVDensityAccuracy;
	}
	else if (EngineShowFlags.OutputMaterialTextureScales) // Test before accuracy is set since accuracy could also be set.
	{
		return DVSM_OutputMaterialTextureScales;
	}
	else if (EngineShowFlags.MaterialTextureScaleAccuracy)
	{
		return DVSM_MaterialTextureScaleAccuracy;
	}
	else if (EngineShowFlags.RequiredTextureResolution)
	{
		return DVSM_RequiredTextureResolution;
	}
	else if (EngineShowFlags.LODColoration || EngineShowFlags.HLODColoration)
	{
		return DVSM_LODColoration;
	}
	else if (EngineShowFlags.VisualizeGPUSkinCache)
	{
		return DVSM_VisualizeGPUSkinCache;
	}
	else if (EngineShowFlags.VisualizeLWCComplexity)
	{
		return DVSM_LWCComplexity;
	}
	else if (EngineShowFlags.VisualizeShadowCasters)
	{
		return DVSM_ShadowCasters;
	}
	else if (EngineShowFlags.StreamingTextureDeficit
	      || EngineShowFlags.StreamingTextureResidency
	      || EngineShowFlags.StreamingMeshLODDeficit
	      || EngineShowFlags.StreamingMeshLODResidency)
	{
		return DVSM_StreamingDeficit;
	}
	return DVSM_None;
}

EDebugViewShaderMode FSceneViewFamily::GetDebugViewShaderMode() const
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	if (DebugViewShaderMode == DVSM_MAX)
	{
		DebugViewShaderMode = ChooseDebugViewShaderMode();
		
		if (!AllowDebugViewShaderMode(DebugViewShaderMode, GetShaderPlatform(), GetFeatureLevel()))
		{
			DebugViewShaderMode = DVSM_None;
		}
	}

	return DebugViewShaderMode;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

#endif // WITH_DEBUG_VIEW_MODES

TPimplPtr<FSceneViewStateSystemMemoryMirror> FSceneViewStateInterface::SystemMemoryMirrorAllocate()
{
	return MakePimpl<FSceneViewStateSystemMemoryMirror>();
}
