// Copyright Epic Games, Inc. All Rights Reserved.

#include "SceneViewExtension.h"
#include "Engine/Engine.h"
#include "Templates/Greater.h"

//
// FSceneViewExtensionBase
//

FSceneViewExtensionBase::FSceneViewExtensionBase(const FAutoRegister&)
{
}

FSceneViewExtensionBase::~FSceneViewExtensionBase()
{
	// The engine stores view extensions by TWeakPtr<ISceneViewExtension>,
	// so they will be automatically unregistered when removed.
}

bool FSceneViewExtensionBase::IsActiveThisFrame(const FSceneViewExtensionContext& Context) const
{
	// Go over any existing activation lambdas
	for (const FSceneViewExtensionIsActiveFunctor& IsActiveFunction : IsActiveThisFrameFunctions)
	{
		TOptional<bool> IsActive = IsActiveFunction(this, Context);

		// If the function does not return a definitive answer, try the next one.
		if (IsActive.IsSet())
		{
			return IsActive.GetValue();
		}
	}

	// Fall back to the validation based on the viewport.
	return IsActiveThisFrame_Internal(Context);
}

//
// FWorldSceneViewExtension
//

FWorldSceneViewExtension::FWorldSceneViewExtension(const FAutoRegister& AutoReg, UWorld* InWorld)
	: FSceneViewExtensionBase(AutoReg)
	, World(InWorld)
{
	check(InWorld != nullptr);
}

bool FWorldSceneViewExtension::IsActiveThisFrame_Internal(const FSceneViewExtensionContext& Context) const
{
	return World == Context.GetWorld();
}

//
// FHMDSceneViewExtension
//

bool FHMDSceneViewExtension::IsActiveThisFrame_Internal(const FSceneViewExtensionContext& Context) const
{
	return Context.IsHMDSupported() && Context.IsStereoSupported();
}

//
// FSceneViewExtensions
//

void FSceneViewExtensions::RegisterExtension(const FSceneViewExtensionRef& RegisterMe)
{
	if (ensure(GEngine))
	{
		auto& KnownExtensions = GEngine->ViewExtensions->KnownExtensions;
		// Compact the list of known extensions.
		for (int32 i = 0; i < KnownExtensions.Num(); )
		{
			if (KnownExtensions[i].IsValid())
			{
				i++;
			}
			else
			{
				KnownExtensions.RemoveAtSwap(i);
			}
		}

		KnownExtensions.AddUnique(RegisterMe);
	}
}

void FSceneViewExtensions::ForEachActiveViewExtension(
	const TArray<TWeakPtr<ISceneViewExtension, ESPMode::ThreadSafe>>& InExtensions,
	const FSceneViewExtensionContext& InContext,
	const TFunctionRef<void(const FSceneViewExtensionRef&)>& Func)
{
	for (const TWeakPtr<ISceneViewExtension, ESPMode::ThreadSafe>& ViewExtPtr : InExtensions)
	{
		TSharedPtr<ISceneViewExtension, ESPMode::ThreadSafe> ViewExt = ViewExtPtr.Pin();
		if (ViewExt.IsValid() && ViewExt->IsActiveThisFrame(InContext))
		{
			Func(ViewExt.ToSharedRef());
		}
	}
}

// @todo viewext : We should cache all the active extensions in OnStartFrame somewhere
const TArray<FSceneViewExtensionRef> FSceneViewExtensions::GatherActiveExtensions(const FSceneViewExtensionContext& InContext) const
{
	TArray<FSceneViewExtensionRef> ActiveExtensions;
	ActiveExtensions.Reserve(KnownExtensions.Num());

	ForEachActiveViewExtension(KnownExtensions, InContext, [&ActiveExtensions](const FSceneViewExtensionRef& ActiveExtension)
		{
			ActiveExtensions.Add(ActiveExtension);
		});

	Algo::SortBy(ActiveExtensions, &ISceneViewExtension::GetPriority, TGreater<>());

	return ActiveExtensions;
}

