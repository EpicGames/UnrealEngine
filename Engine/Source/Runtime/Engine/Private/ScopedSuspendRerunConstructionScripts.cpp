// Copyright Epic Games, Inc. All Rights Reserved.

#include "ScopedSuspendRerunConstructionScripts.h"

#if WITH_EDITOR

int32 FScopedSuspendRerunConstructionScripts::SuspensionCount = 0;
TSet<TWeakObjectPtr<AActor>> FScopedSuspendRerunConstructionScripts::PendingActors;

FScopedSuspendRerunConstructionScripts::FScopedSuspendRerunConstructionScripts()
{
	++SuspensionCount;
}

FScopedSuspendRerunConstructionScripts::~FScopedSuspendRerunConstructionScripts()
{
	--SuspensionCount;
	// When all scopes have been exited, fire off any deferred reruns
	if (SuspensionCount == 0 && PendingActors.Num() > 0)
	{
		// Make a local copy; we'll clear the set before we run scripts,
		// so if any subsequent rerun calls happen, they get queued again.
		TArray<TWeakObjectPtr<AActor>> ActorsToRerun = PendingActors.Array();
		PendingActors.Empty();

		for (TWeakObjectPtr<AActor> WeakActor : ActorsToRerun)
		{
			if (AActor* Actor = WeakActor.Get())
			{
				Actor->RerunConstructionScripts();
			}
		}
	}
}

void FScopedSuspendRerunConstructionScripts::DeferRerun(AActor* Actor)
{
	if (Actor)
	{
		PendingActors.Add(Actor);
	}
}

#endif