// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	ScreenRendering.cpp: Screen rendering implementation.
=============================================================================*/

#include "ScreenRendering.h"

/** Vertex declaration for screen-space rendering. */
TGlobalResource<FScreenVertexDeclaration> GScreenVertexDeclaration;

// Shader implementations.
IMPLEMENT_SHADER_TYPE(, FScreenPS, TEXT("/Engine/Private/ScreenPixelShader.usf"), TEXT("Main"), SF_Pixel);
IMPLEMENT_SHADER_TYPE(, FScreenFromSlice0PS, TEXT("/Engine/Private/ScreenPixelShader.usf"), TEXT("Main"), SF_Pixel);
IMPLEMENT_SHADER_TYPE(, FScreenUnwrapSlicesPS, TEXT("/Engine/Private/ScreenPixelShader.usf"), TEXT("MainUnwrap"), SF_Pixel);
IMPLEMENT_SHADER_TYPE(,FScreenPSInvertAlpha,TEXT("/Engine/Private/ScreenPixelShader.usf"),TEXT("MainInvertAlpha"),SF_Pixel);
IMPLEMENT_SHADER_TYPE(,FScreenPSsRGBSource, TEXT("/Engine/Private/ScreenPixelShader.usf"), TEXT("MainsRGBSource"), SF_Pixel);
IMPLEMENT_SHADER_TYPE(,FScreenPSArraySlice, TEXT("/Engine/Private/ScreenPixelShader.usf"), TEXT("Main"), SF_Pixel);
IMPLEMENT_SHADER_TYPE(,FScreenPSMipLevel, TEXT("/Engine/Private/ScreenPixelShader.usf"), TEXT("MainMipLevel"), SF_Pixel);
IMPLEMENT_SHADER_TYPE(,FScreenPSsRGBSourceMipLevel, TEXT("/Engine/Private/ScreenPixelShader.usf"), TEXT("MainsRGBSourceMipLevel"), SF_Pixel);
IMPLEMENT_SHADER_TYPE(,FScreenVS,TEXT("/Engine/Private/ScreenVertexShader.usf"),TEXT("Main"),SF_Vertex);
IMPLEMENT_SHADER_TYPE(,FScreenPS_OSE,TEXT("/Engine/Private/ScreenPixelShaderOES.usf"),TEXT("Main"),SF_Pixel);
