// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	ScriptPlatformInterface.cpp: Base functionality for the various script accessible platform-interface code
=============================================================================*/ 

#include "Misc/FileHelper.h"
#include "Serialization/MemoryWriter.h"
#include "Serialization/MemoryReader.h"
#include "Misc/ConfigCacheIni.h"
#include "UObject/Package.h"
#include "Serialization/ObjectAndNameAsStringProxyArchive.h"
#include "Engine/PlatformInterfaceBase.h"
#include "Templates/SubclassOf.h"
#include "Engine/CloudStorageBase.h"
#include "Engine/MicroTransactionBase.h"
#include "Engine/PlatformInterfaceWebResponse.h"
#include "Engine/TwitterIntegrationBase.h"
#include "Engine/InGameAdManager.h"
#include "UObject/WeakObjectPtr.h"

DEFINE_LOG_CATEGORY_STATIC(LogScriptPlatformInterface, Log, All);

/*******************************************
 * Platform Interface Base                 *
 *******************************************/
UPlatformInterfaceBase::UPlatformInterfaceBase(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}


bool UPlatformInterfaceBase::HasDelegates(int32 DelegateType)
{
	// has script ever put anything in for this delegate type?
	if (AllDelegates.Num() > DelegateType)
	{
		// if so are there currently any set?
		if (AllDelegates[DelegateType].Delegates.Num() > 0)
		{
			return true;
		}
	}

	return false;
}


void UPlatformInterfaceBase::CallDelegates(int32 DelegateType, FPlatformInterfaceDelegateResult& Result)
{
	// make sure that script has ever put anything in for this delegate type
	if (AllDelegates.Num() <= DelegateType)
	{
		return;
	}

	// call all the delegates that are set
	FDelegateArray& DelegateArray = AllDelegates[DelegateType];
	
	// copy the array in case delegates are removed from the class's delegates array
	TArray<FPlatformInterfaceDelegate> ArrayCopy = DelegateArray.Delegates;
	for (int32 DelegateIndex = 0; DelegateIndex < ArrayCopy.Num(); DelegateIndex++)
	{
		ArrayCopy[ DelegateIndex ].ExecuteIfBound( Result );
	}
}


bool UPlatformInterfaceBase::StaticExec(const TCHAR* Cmd, FOutputDevice& Ar)
{
	// @todo: Ask each subclass via a static function like GetExecBaseCommand(), then
	// get the singleton, and pass the rest of the command to it
	if (FParse::Command( &Cmd, TEXT("Ad")))
	{
		UInGameAdManager* AdManager = UPlatformInterfaceBase::GetInGameAdManagerSingleton();
		if (FParse::Command(&Cmd, TEXT("Show")))
		{
			AdManager->ShowBanner(!!FCString::Atoi(Cmd));
		}
		else if (FParse::Command(&Cmd, TEXT("Hide")))
		{
			AdManager->HideBanner();
		}
		else if (FParse::Command(&Cmd, TEXT("Close")))
		{
			AdManager->ForceCloseAd();
		}
		return true;
	}

	return false;
}


void UPlatformInterfaceBase::AddDelegate(int32 DelegateType, FPlatformInterfaceDelegate InDelegate)
{
	if (AllDelegates.Num() < DelegateType + 1)
	{
		for (int32 i = 0; i < DelegateType + 1 - AllDelegates.Num(); ++i)
		{
			AllDelegates.Add(FDelegateArray());
		}
	}
	// Add this delegate to the array if not already present
	if (AllDelegates[DelegateType].Delegates.Find(InDelegate) == INDEX_NONE)
	{
		AllDelegates[DelegateType].Delegates.Add(InDelegate);
	}
}


void UPlatformInterfaceBase::ClearDelegate(int32 DelegateType, FPlatformInterfaceDelegate InDelegate)
{
	int32 RemoveIndex;

	if (DelegateType < AllDelegates.Num())
	{
		// Remove this delegate from the array if found
		RemoveIndex = AllDelegates[DelegateType].Delegates.Find(InDelegate);
		if (RemoveIndex != INDEX_NONE)
		{
			AllDelegates[DelegateType].Delegates.RemoveAt(RemoveIndex,1);
		}
	}
}




#define IMPLEMENT_PLATFORM_INTERFACE_SINGLETON(Class, ClassDesc) \
	/** C++ interface to get the singleton */ \
	Class* UPlatformInterfaceBase::Get##ClassDesc##Singleton() \
	{ \
		/* the singleton object */ \
		static Class* Singleton = NULL; \
 \
		/* have we created the singleton yet? */ \
		if (Singleton == NULL) \
		{ \
			/* load the class name from the .ini */ \
			FString SingletonClassName; \
			GConfig->GetString(TEXT("PlatformInterface"), TEXT(#ClassDesc) TEXT("ClassName"), SingletonClassName, GEngineIni); \
 \
			/* load the class (most likely intrinsic) */ \
			UClass* SingletonClass = LoadClass<Class>(NULL, *SingletonClassName, NULL, LOAD_None, NULL); \
			if (SingletonClass == NULL) \
			{ \
				/* if something failed, then use the default */ \
				SingletonClass = Class::StaticClass(); \
			} \
 \
			/* make the singleton object */ \
			Singleton = NewObject<Class>(GetTransientPackage(), SingletonClass); \
			check(Singleton); \
 \
			/* initialize it */ \
			Singleton->Init(); \
		} \
 \
		/* there will be a Singleton by this point (or an error above) */ \
		return Singleton; \
	} \
 \
	 /** This is called on the default object, call the class static function */ \
	Class* UPlatformInterfaceBase::Get##ClassDesc() \
	{ \
		return Get##ClassDesc##Singleton(); \
	}


IMPLEMENT_PLATFORM_INTERFACE_SINGLETON(UCloudStorageBase, CloudStorageInterface)
IMPLEMENT_PLATFORM_INTERFACE_SINGLETON(UInGameAdManager, InGameAdManager)
IMPLEMENT_PLATFORM_INTERFACE_SINGLETON(UMicroTransactionBase, MicroTransactionInterface)
IMPLEMENT_PLATFORM_INTERFACE_SINGLETON(UTwitterIntegrationBase,TwitterIntegration)



/*******************************************
 * Cloud Storage                           *
 *******************************************/

UCloudStorageBase::UCloudStorageBase(const FObjectInitializer& ObjectInitializer)
 : Super(ObjectInitializer)
{
}


void UCloudStorageBase::Init()
{

}

bool UCloudStorageBase::ReadKeyValue(const FString& KeyName, EPlatformInterfaceDataType Type, class UObject* SerializedObj)
{
	// we are going to call the result delegates right away, so just fill out a result
	FPlatformInterfaceDelegateResult Result;
	FMemory::Memzero(&Result, sizeof(Result));

	Result.bSuccessful = true;
	Result.Data.Type = Type;

	GConfig->EnableFileOperations();
	static FString CloudStorageIni(FPaths::CloudDir() + TEXT("CloudStorage.ini"));
	switch (Type)
	{
		case PIDT_Int:
			GConfig->GetInt(TEXT("CloudStorageEmulation"), *KeyName, Result.Data.IntValue, *CloudStorageIni);
			break;
		case PIDT_Float:
			GConfig->GetFloat(TEXT("CloudStorageEmulation"), *KeyName, Result.Data.FloatValue, *CloudStorageIni);
			break;
		case PIDT_String:
			GConfig->GetString(TEXT("CloudStorageEmulation"), *KeyName, Result.Data.StringValue, *CloudStorageIni);
			break;
		case PIDT_Object:
			{
				UE_LOGF(LogScriptPlatformInterface, Log, "CONVERT SOME uint8 ARRAY TO UOBJECT HERE?");
			}
			break;
		case PIDT_Custom:
			break;
	}
	GConfig->DisableFileOperations();

	// tell script we've read the data
	CallDelegates(CSD_KeyValueReadComplete, Result);

	return true;
}

bool UCloudStorageBase::WriteKeyValue(const FString&  KeyName, const FPlatformInterfaceData& Value)
{
	static FString CloudStorageIni(FPaths::CloudDir() + TEXT("CloudStorage.ini"));

	GConfig->EnableFileOperations();
	switch (Value.Type)
	{
		case PIDT_Int:
			GConfig->SetInt(TEXT("CloudStorageEmulation"), *KeyName, Value.IntValue, *CloudStorageIni);
			break;
		case PIDT_Float:
			GConfig->SetFloat(TEXT("CloudStorageEmulation"), *KeyName, Value.FloatValue, *CloudStorageIni);
			break;
		case PIDT_String:
			GConfig->SetString(TEXT("CloudStorageEmulation"), *KeyName, *Value.StringValue, *CloudStorageIni);
			break;
		case PIDT_Object:
			{
				UE_LOGF(LogScriptPlatformInterface, Log, "CONVERT SOME uint8 ARRAY TO UOBJECT HERE");
			}
			break;
	};
	// write it out
	GConfig->Flush(false, *CloudStorageIni);
	GConfig->DisableFileOperations();

	FPlatformInterfaceDelegateResult Result;
	Result.bSuccessful = true;
	Result.Data = Value;

	// tell script we've written the data, successfully
	CallDelegates(CSD_KeyValueWriteComplete, Result);

	return true;
}


bool UCloudStorageBase::QueryForCloudDocuments()
{
	// look for the files
	IFileManager::Get().FindFilesRecursive(LocalCloudFiles, *FPaths::CloudDir(), TEXT("*.*"), true, false);

	if (!bSuppressDelegateCalls)
	{
		// and we're done, call the delegates
		FPlatformInterfaceDelegateResult Result;
		Result.bSuccessful = true;
		CallDelegates(CSD_DocumentQueryComplete, Result);
	}
	return true;
}


int32 UCloudStorageBase::GetNumCloudDocuments(bool bIsForConflict)
{
	if (bIsForConflict)
	{
		return 0;
	}
	return LocalCloudFiles.Num();
}


FString UCloudStorageBase::GetCloudDocumentName(int32 Index)
{
	// verify the input
	if (Index >= LocalCloudFiles.Num())
	{
		return FString();
	}

	// pull apart the URL to get the filename
  	return FPaths::GetCleanFilename(LocalCloudFiles[Index]);
}


int32 UCloudStorageBase::CreateCloudDocument(const FString& Filename)
{
	FString FinalFilename = FPaths::CloudDir() + Filename;
	return LocalCloudFiles.Add(FinalFilename);
}


bool UCloudStorageBase::ReadCloudDocument(int32 Index, bool bIsForConflict)
{
	// verify the input
	if (bIsForConflict || Index >= LocalCloudFiles.Num())
	{
		return false;
	}

	// just call the delegate, we'll read in in the Parse function
	if (IFileManager::Get().FileSize(*LocalCloudFiles[Index]) != -1)
	{
		if (!bSuppressDelegateCalls)
		{
			FPlatformInterfaceDelegateResult Result;
			Result.bSuccessful = true;
			// which document is this?
			Result.Data.Type = PIDT_Int;
			Result.Data.IntValue = Index;
			CallDelegates(CSD_DocumentReadComplete, Result);
		}		

		return true;
	}
	return false;
}


FString UCloudStorageBase::ParseDocumentAsString(int32 Index, bool bIsForConflict)
{
	// verify the input
	if (bIsForConflict || Index >= LocalCloudFiles.Num())
	{
		return FString();
	}

	FString Result;
	FFileHelper::LoadFileToString(Result, *LocalCloudFiles[Index]);
	return Result;
}


void UCloudStorageBase::ParseDocumentAsBytes(int32 Index, TArray<uint8>& ByteData, bool bIsForConflict)
{
	// make sure a clean slate
	ByteData.Empty();

	// verify the input
	if (bIsForConflict || Index >= LocalCloudFiles.Num())
	{
		return;
	}

	FFileHelper::LoadFileToArray(ByteData, *LocalCloudFiles[Index]);
}


UObject* UCloudStorageBase::ParseDocumentAsObject(int32 Index, TSubclassOf<class UObject> ObjectClass, int32 ExpectedVersion, bool bIsForConflict)
{
	TArray<uint8> ObjectBytes;
	// read in a uint8 array
	ParseDocumentAsBytes(Index, ObjectBytes, bIsForConflict);

	// make sure we got some bytes
	if (ObjectBytes.Num() == 0)
	{
		return NULL;
	}

	FMemoryReader MemoryReader(ObjectBytes, true);

	// load the version the object was saved with
	int32 SavedVersion;
	MemoryReader << SavedVersion;

	// make sure it matches
	if (SavedVersion != ExpectedVersion)
	{
		// note that it failed to read
		UE_LOGF(LogScriptPlatformInterface, Log, "Load failed: Cloud document was saved with an incompatibly version (%d, expected %d).", SavedVersion, ExpectedVersion);
		return NULL;
	}

	// use a wrapper archive that converts FNames and UObject*'s to strings that can be read back in
	FObjectAndNameAsStringProxyArchive Ar(MemoryReader, false);

	// NOTE: The following should be in shared functionality in UCloudStorageBase
	// create the object
	UObject* Obj = NewObject<UObject>(GetTransientPackage(), ObjectClass);

	// serialize the object
	Obj->Serialize(Ar);

	// return the deserialized object
	return Obj;
}




bool UCloudStorageBase::WriteCloudDocument(int32 Index)
{
	// verify the input
	if (Index >= LocalCloudFiles.Num())
	{
		return false;
	}

	if (!bSuppressDelegateCalls)
	{
		// was already written out in the SaveDocument, so just call the delegate
		FPlatformInterfaceDelegateResult Result;
		Result.bSuccessful = true;
		// which document is this?
		Result.Data.Type = PIDT_Int;
		Result.Data.IntValue = Index;
		CallDelegates(CSD_DocumentWriteComplete, Result);
	}
	return true;
}


bool UCloudStorageBase::SaveDocumentWithString(int32 Index, const FString& StringData)
{
	// verify the input
	if (Index >= LocalCloudFiles.Num())
	{
		return false;
	}

	return FFileHelper::SaveStringToFile(StringData, *LocalCloudFiles[Index]);
}



bool UCloudStorageBase::SaveDocumentWithBytes(int32 Index, const TArray<uint8>& ByteData)
{
	// verify the input
	if (Index >= LocalCloudFiles.Num())
	{
		return false;
	}

	return FFileHelper::SaveArrayToFile(ByteData, *LocalCloudFiles[Index]);
}


bool UCloudStorageBase::SaveDocumentWithObject(int32 Index, UObject* ObjectData, int32 SaveVersion)
{
	// verify the input
	if (GetCloudDocumentName(Index) == TEXT(""))
	{
		return false;
	}

	TArray<uint8> ObjectBytes;
	FMemoryWriter MemoryWriter(ObjectBytes);

	// save out a version
	MemoryWriter << SaveVersion;

	// use a wrapper archive that converts FNames and UObject*'s to strings that can be read back in
	FObjectAndNameAsStringProxyArchive Ar(MemoryWriter, false);

	// serialize the object
	ObjectData->Serialize(Ar);

	// now, push the byte array into the document
	SaveDocumentWithBytes(Index, ObjectBytes);

	return true;
}


bool UCloudStorageBase::ResolveConflictWithNewestDocument()
{
	return false;
}


bool UCloudStorageBase::ResolveConflictWithVersionIndex(int32 /*Index*/)
{
	return false;
}





/*******************************************
 * Microtransactions                       *
 *******************************************/


UMicroTransactionBase::UMicroTransactionBase(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

void UMicroTransactionBase::Init()
{

}

bool UMicroTransactionBase::QueryForAvailablePurchases()
{
	return false;
}


bool UMicroTransactionBase::IsAllowedToMakePurchases()
{
	return false;
}


bool UMicroTransactionBase::BeginPurchase(int Index)
{
	return false;
}



/*******************************************
 * Twitter Integration
 *******************************************/
UTwitterIntegrationBase::UTwitterIntegrationBase(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}


void UTwitterIntegrationBase::Init()
{
}


bool UTwitterIntegrationBase::AuthorizeAccounts()
{
	return false;
}


int32 UTwitterIntegrationBase::GetNumAccounts()
{
	return 0;
}


FString UTwitterIntegrationBase::GetAccountName(int AccountIndex)
{
	return TEXT("");
}


bool UTwitterIntegrationBase::CanShowTweetUI()
{
	return false;
}


bool UTwitterIntegrationBase::ShowTweetUI(const FString& InitialMessage, const FString& URL, const FString& Picture)
{
	return false;
}


bool UTwitterIntegrationBase::TwitterRequest(const FString& URL, const TArray<FString>& ParamKeysAndValues, ETwitterRequestMethod RequestMethod, int32 AccountIndex)
{
	return false;
}


/*******************************************
 * Platform Interface Web Response         *
 *******************************************/
UPlatformInterfaceWebResponse::UPlatformInterfaceWebResponse(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

/** @return the number of header/value pairs */
int32 UPlatformInterfaceWebResponse::GetNumHeaders()
{
	return Headers.Num();
}

/** Retrieve the header and value for the given index of header/value pair */
void UPlatformInterfaceWebResponse::GetHeader(int32 HeaderIndex,FString& Header,FString& Value)
{
	// this is slow if script iterates over the map one at a time, but it's not expected this will be called often
	int32 Index = 0;
	for (TMap<FString,FString>::TIterator It(Headers); It && Index < Headers.Num(); ++It, ++Index)
	{
		// is it the requested header?
		if (Index == HeaderIndex)
		{
			Header = It.Key();
			Value = It.Value();
		}
	}
}

/** @return the value for the given header (or "" if no matching header) */
FString UPlatformInterfaceWebResponse::GetHeaderValue(const FString& HeaderName)
{
	// look up the header
	FString* Val = Headers.Find(HeaderName);
	if (Val)
	{
		// return if it exists
		return *Val;
	}
	return TEXT("");
}
