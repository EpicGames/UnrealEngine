// Copyright Epic Games, Inc. All Rights Reserved.

#include "Engine/ScriptViewportClient.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ScriptViewportClient)

UScriptViewportClient::UScriptViewportClient(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

UWorld* UScriptViewportClient::GetWorld() const
{
	return Super::GetWorld();
}
