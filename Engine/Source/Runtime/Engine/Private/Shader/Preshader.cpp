// Copyright Epic Games, Inc. All Rights Reserved.
#include "Shader/Preshader.h"
#include "Misc/DelayedAutoRegister.h"
#include "Shader/PreshaderEvaluate.h"
#include "Materials/MaterialRenderProxy.h"
#include "Materials/MaterialUniformExpressions.h"
#include "Engine/Texture.h"
#include "Shader/PreshaderTypes.h"
#include "SparseVolumeTexture/SparseVolumeTexture.h"
#include "VT/RuntimeVirtualTexture.h"
#include "Hash/xxhash.h"
#include "ExternalTexture.h"
#include "TextureResource.h"

IMPLEMENT_TYPE_LAYOUT(UE::Shader::FPreshaderData);
IMPLEMENT_TYPE_LAYOUT(UE::Shader::FPreshaderStructType);

namespace UE
{
namespace Shader
{

// For prettier debug printing of preshaders, we'd like add / subtract / multiply / divide to be operators instead of function calls.  To
// implement this, we need to know when a given operator requires parentheses for correct order of operations.  We track the type of
// expression for each stack element, and can decide whether an operator is valid without parentheses based on the state of the elements:
//
//		Singular,		Singular		+-*/ valid
//		Singular,		SumTerms		+    valid
//		Singular,		ProductTerms	+-*  valid
//		SumTerms,		Singular		+-   valid
//		SumTerms,		SumTerms		+    valid
//		SumTerms,		ProductTerms	+-   valid
//		ProductTerms,	Singular		+-*/ valid
//		ProductTerms,	SumTerms		+    valid
//		ProductTerms,	ProductTerms	+-*  valid
//
// Besides the above, swizzles require parentheses for non-singular elements, so we can use the state for that purpose as well.  Other
// operators (like comparison), are quite rare in Preshaders, so it's not worth the effort to try to convert those.
//
enum class EPreshaderDebugStackType : uint8
{
	Singular,				// Stack element is a singular term (constant, param, function, parentheses wrapped, etc)
	SumTerms,				// Stack element is a series of added / subtracted terms
	ProductTerms,			// Stack element is a series of multiplied / divided terms
};

enum class EPreshaderOperatorType : uint8
{
	Add,
	Sub,
	Mul,
	Div,
	None
};

// Each entry in the table is the highest operator enum value that's valid without parentheses for that state combination.
// As you can see from the valid combinations listed in the comment above, valid combinations are all monotonic sequences,
// which is why we only need to store the maximum valid operator type, as opposed to flags.
static const EPreshaderOperatorType PreshaderDebugStackLookup[3][3] =
{
	// Second element:
	// Singular						SumTerms						ProductTerms
	{ EPreshaderOperatorType::Div,	EPreshaderOperatorType::Add,	EPreshaderOperatorType::Mul },		// First element:  Singular
	{ EPreshaderOperatorType::Sub,	EPreshaderOperatorType::Add,	EPreshaderOperatorType::Sub },		// First element:  SumTerms
	{ EPreshaderOperatorType::Div,	EPreshaderOperatorType::Add,	EPreshaderOperatorType::Mul },		// First element:  ProductTerms
};

struct FPreshaderDebugStackElement
{
	FString Text;
	EPreshaderDebugStackType DebugStackType;
};

struct FPreshaderDebugStack
{
	TArray<FPreshaderDebugStackElement> Elements;
	TMap<FString, uint32>* ParameterReferences = nullptr;
	uint32 OpCount = 0;
	uint32 ParameterCount = 0;
	bool bGenerateString = true;

	void Push(FString Text, EPreshaderDebugStackType DebugStackType)
	{
		Elements.Push({ MoveTemp(Text), DebugStackType });
	}

	void Reset()
	{
		Elements.Reset();
		// By design, doesn't reset "ParameterReferences", so they can be accumulated for the caller
	}
};

FPreshaderType::FPreshaderType(EValueType InType) : ValueType(InType)
{
}

EValueComponentType FPreshaderType::GetComponentType(int32 Index) const
{
	const FValueTypeDescription& TypeDesc = GetValueTypeDescription(ValueType);
	return (Index >= 0 && Index < TypeDesc.NumComponents) ? TypeDesc.ComponentType : EValueComponentType::Void;
}

void FPreshaderStack::PushValue(const FValue& InValue)
{
	check(InValue.Component.Num() == InValue.Type.GetNumComponents());
	Values.Emplace(InValue.Type);
	Components.Append(InValue.Component);
}

void FPreshaderStack::PushValue(const FPreshaderValue& InValue)
{
	check(InValue.Component.Num() == InValue.Type.GetNumComponents());
	Values.Add(InValue.Type);
	Components.Append(InValue.Component.GetData(), InValue.Component.Num());
}

void FPreshaderStack::PushValue(const FPreshaderType& InType, TArrayView<const FValueComponent> InComponents)
{
	check(InComponents.Num() == InType.GetNumComponents());
	Values.Add(InType);
	Components.Append(InComponents.GetData(), InComponents.Num());
}

TArrayView<FValueComponent> FPreshaderStack::PushEmptyValue(const FPreshaderType& InType)
{
	Values.Add(InType);
	const int32 NumComponents = InType.GetNumComponents();
	const int32 ComponentIndex = Components.AddZeroed(NumComponents);
	return MakeArrayView(Components.GetData() + ComponentIndex, NumComponents);
}

FValueComponent* FPreshaderStack::PushEmptyValue(EValueType InType, int32 NumComponents)
{
	Values.Add(InType);
	checkSlow(NumComponents == Values.Last().GetNumComponents());
	const int32 ComponentIndex = Components.AddZeroed(NumComponents);
	return Components.GetData() + ComponentIndex;
}

FPreshaderValue FPreshaderStack::PopValue()
{
	FPreshaderValue Value;
	Value.Type = Values.Pop(EAllowShrinking::No);

	const int32 NumComponents = Value.Type.GetNumComponents();
	const int32 ComponentIndex = Components.Num() - NumComponents;

	Value.Component = MakeArrayView(Components.GetData() + ComponentIndex, NumComponents);
	Components.RemoveAt(ComponentIndex, NumComponents, EAllowShrinking::No);

	return Value;
}

void FPreshaderStack::PopResult(FPreshaderValue& OutValue)
{
	if (Values.Num() > 0)
	{
		check(Values.Num() == 1);
		check(Values[0].GetNumComponents() == Components.Num());

		OutValue.Type = Values[0];
		OutValue.Component = Components;

		// Call Reset instead of Empty, so it doesn't change the memory allocation
		Values.Reset();
		Components.Reset();
	}
}

FPreshaderValue FPreshaderStack::PeekValue(int32 Offset)
{
	FPreshaderValue Value;
	Value.Type = Values.Last(Offset);

	int32 OffsetComponents = 0;
	for (int32 OffsetIndex = 0; OffsetIndex < Offset; OffsetIndex++)
	{
		OffsetComponents += Values.Last(OffsetIndex).GetNumComponents();
	}

	const int32 NumComponents = Value.Type.GetNumComponents();
	const int32 ComponentIndex = Components.Num() - NumComponents - OffsetComponents;
	check(ComponentIndex >= 0);
	Value.Component = MakeArrayView(Components.GetData() + ComponentIndex, NumComponents);
	return Value;
}

FValue FPreshaderValue::AsShaderValue(const FStructTypeRegistry* TypeRegistry) const
{
	FValue Result;
	Result.Type = Type.ValueType;
	Result.Component.Append(Component.GetData(), Component.Num());
	return Result;
}

void FPreshaderData::WriteData(const void* Value, uint32 Size)
{
	Data.Append((uint8*)Value, Size);
}

void FPreshaderData::WriteName(const FScriptName& Name)
{
	int32 Index = Names.Find(Name);
	if (Index == INDEX_NONE)
	{
		Index = Names.Add(Name);
	}

	check(Index >= 0 && Index <= 0xffff);
	Write((uint16)Index);
}

void FPreshaderData::WriteValue(const FValue& Value)
{
	const int32 NumComponents = Value.Type.GetNumComponents();

	check(!Value.Type.IsObject() && !Value.Type.IsStruct());

	Write(Value.Type.ValueType);
	for (int32 Index = 0; Index < NumComponents; ++Index)
	{
		const EValueComponentType ComponentType = Value.Type.GetComponentType(Index);
		const int32 ComponentSize = GetComponentTypeSizeInBytes(ComponentType);
		const FValueComponent Component = Value.GetComponent(Index);
		WriteData(&Component.Packed, ComponentSize);
	}
}

FPreshaderDataContext::FPreshaderDataContext(const FPreshaderData& InData, TArrayView<float> InResultsBuffer)
	: Ptr(InData.Data.GetData())
	, EndPtr(Ptr + InData.Data.Num())
	, Names(InData.Names)
	, ResultsBuffer(InResultsBuffer)
{}

FPreshaderDataContext::FPreshaderDataContext(const FPreshaderDataContext& InContext, uint32 InOffset, uint32 InSize)
	: Ptr(InContext.Ptr + InOffset)
	, EndPtr(Ptr + InSize)
	, Names(InContext.Names)
{
	// Preshader2 doesn't include individual preshaders, just a single bytecode stream, so this function shouldn't be called on those.
	check(InContext.IsPreshader2() == false);
}

static inline void ReadPreshaderData(FPreshaderDataContext& RESTRICT Data, int32 Size, void* Result)
{
	FMemory::Memcpy(Result, Data.Ptr, Size);
	Data.Ptr += Size;
	checkSlow(Data.Ptr <= Data.EndPtr);
}

template<typename T>
static inline T ReadPreshaderValue(FPreshaderDataContext& RESTRICT Data)
{
	T Result;
	ReadPreshaderData(Data, sizeof(T), &Result);
	return Result;
}

template<>
inline uint8 ReadPreshaderValue<uint8>(FPreshaderDataContext& RESTRICT Data)
{
	checkSlow(Data.Ptr < Data.EndPtr);
	return *Data.Ptr++;
}

template<>
FScriptName ReadPreshaderValue<FScriptName>(FPreshaderDataContext& RESTRICT Data)
{
	const int32 Index = ReadPreshaderValue<uint16>(Data);
	return Data.Names[Index];
}

template<>
FPreshaderType ReadPreshaderValue<FPreshaderType>(FPreshaderDataContext& RESTRICT Data)
{
	FPreshaderType Result;
	Result.ValueType = ReadPreshaderValue<EValueType>(Data);
	return Result;
}

template<>
FName ReadPreshaderValue<FName>(FPreshaderDataContext& RESTRICT Data) = delete;

template<>
FHashedMaterialParameterInfo ReadPreshaderValue<FHashedMaterialParameterInfo>(FPreshaderDataContext& RESTRICT Data)
{
	const FScriptName Name = ReadPreshaderValue<FScriptName>(Data);
	const int32 Index = ReadPreshaderValue<int32>(Data);
	const TEnumAsByte<EMaterialParameterAssociation> Association = ReadPreshaderValue<TEnumAsByte<EMaterialParameterAssociation>>(Data);
	return FHashedMaterialParameterInfo(Name, Association, Index);
}

static void EvaluateConstantZero(FPreshaderStack& Stack, FPreshaderDataContext& RESTRICT Data)
{
	const FPreshaderType Type = ReadPreshaderValue<FPreshaderType>(Data);
	Stack.PushEmptyValue(Type); // Leave the empty value zero-initialized
}

static void EvaluateConstantZero(FPreshaderDebugStack& Stack, FPreshaderDataContext& RESTRICT Data)
{
	Stack.OpCount++;

	FPreshaderStack TempStack;
	EvaluateConstantZero(TempStack, Data);

	if (Stack.bGenerateString)
	{
		// Display scalar constants without type information, for better readability
		bool bIsScalar = TempStack.PeekValue().Type.ValueType == EValueType::Float1;
		TStringBuilder<256> ValueBuffer;
		Stack.Push(TempStack.PeekValue().AsShaderValue().ToString(bIsScalar ? EValueStringFormat::Description : EValueStringFormat::HLSL, ValueBuffer), EPreshaderDebugStackType::Singular);
	}
}

static void EvaluateConstant(FPreshaderStack& Stack, FPreshaderDataContext& RESTRICT Data)
{
	const FPreshaderType Type = ReadPreshaderValue<FPreshaderType>(Data);
	TArrayView<FValueComponent> Component = Stack.PushEmptyValue(Type);

	const FValueTypeDescription& TypeDesc = GetValueTypeDescription(Type.ValueType);
	const int32 ComponmentSizeInBytes = TypeDesc.ComponentSizeInBytes;
	for (int32 Index = 0; Index < Component.Num(); ++Index)
	{
		ReadPreshaderData(Data, ComponmentSizeInBytes, &Component[Index].Packed);
	}
}

static void EvaluateConstant(FPreshaderDebugStack& Stack, FPreshaderDataContext& RESTRICT Data)
{
	Stack.OpCount++;

	FPreshaderStack TempStack;
	EvaluateConstant(TempStack, Data);

	if (Stack.bGenerateString)
	{
		// Display scalar constants without type information, for better readability
		bool bIsScalar = TempStack.PeekValue().Type.ValueType == EValueType::Float1;
		TStringBuilder<256> ValueBuffer;
		Stack.Push(TempStack.PeekValue().AsShaderValue().ToString(bIsScalar ? EValueStringFormat::Description : EValueStringFormat::HLSL, ValueBuffer), EPreshaderDebugStackType::Singular);
	}
}

static void EvaluatePushValue(FPreshaderStack& Stack, FPreshaderDataContext& RESTRICT Data)
{
	const int32 StackOffset = ReadPreshaderValue<uint16>(Data);
	const FPreshaderValue Value = Stack.PeekValue(StackOffset);
	// Make a local copy of the component array, as it will be invalidated when pushing the copy
	const TArray<FValueComponent, TInlineAllocator<64>> LocalComponent(Value.Component);
	Stack.PushValue(Value.Type, LocalComponent);
}

static void EvaluatePushValue(FPreshaderDebugStack& Stack, FPreshaderDataContext& RESTRICT Data)
{
	Stack.OpCount++;

	const int32 StackOffset = ReadPreshaderValue<uint16>(Data);
	if (Stack.bGenerateString)
	{
		Stack.Elements.Push(Stack.Elements[StackOffset]);
	}
}

static void EvaluateAssign(FPreshaderStack& Stack)
{
	const FPreshaderValue Value = Stack.PopValue();
	// Make a local copy of the component array, as it will be invalidated when pushing the copy
	const TArray<FValueComponent, TInlineAllocator<64>> LocalComponent(Value.Component);

	// Remove the old value
	Stack.PopValue();
	// Replace with the new value
	Stack.PushValue(Value.Type, LocalComponent);
}

static void EvaluateAssign(FPreshaderDebugStack& Stack)
{
	Stack.OpCount++;

	if (Stack.bGenerateString)
	{
		FPreshaderDebugStackElement Value = Stack.Elements.Pop();
		Stack.Elements.Pop();
		Stack.Elements.Push(Value);
	}
}

static void EvaluateParameter(FPreshaderStack& Stack, const FUniformExpressionSet* UniformExpressionSet, uint32 ParameterIndex, const FMaterialRenderContext& Context)
{
	if (!UniformExpressionSet)
	{
		// return 0 for parameters if we don't have UniformExpressionSet
		Stack.PushEmptyValue(EValueType::Float1);
		return;
	}

	const FMaterialNumericParameterInfo& Parameter = UniformExpressionSet->GetNumericParameter(ParameterIndex);
	bool bFoundParameter = false;

	// First allow proxy the chance to override parameter
	if (Context.MaterialRenderProxy)
	{
		UE::Shader::FValue ParameterShaderValue;
		if (Context.MaterialRenderProxy->GetParameterShaderValue(Parameter.ParameterType, Parameter.ParameterInfo, ParameterShaderValue, Context))
		{
			Stack.PushValue(ParameterShaderValue);
			bFoundParameter = true;
		}
	}

	// Editor overrides
#if WITH_EDITOR
	if (!bFoundParameter)
	{
		FValue OverrideValue;
		if (Context.Material.TransientOverrides.GetNumericOverride(Parameter.ParameterType, Parameter.ParameterInfo, OverrideValue))
		{
			Stack.PushValue(OverrideValue);
			bFoundParameter = true;
		}
	}
#endif // WITH_EDITOR

	// Default value
	if (!bFoundParameter)
	{
		// Fast path for numeric parameters (common case).  Writes data in place to FPreshaderStack, and has a single
		// lookup table to fetch type and component information.
		if (Parameter.ParameterType <= EMaterialParameterType::DoubleVector)
		{
			// Validate that it's OK to do the comparison above, and that our table is correct
			static_assert((int32)EMaterialParameterType::Scalar == 0);
			static_assert((int32)EMaterialParameterType::Vector == 1);
			static_assert((int32)EMaterialParameterType::DoubleVector == 2);
			static_assert(sizeof(FValueComponent) == sizeof(double));

			// Lookup includes type, number of components, and size to copy per component.
			static const int8 GParameterCopyLookup[3][3] =
			{
				{ (int8)EValueType::Float1, 1, sizeof(float) },		// EMaterialParameterType::Scalar
				{ (int8)EValueType::Float4, 4, sizeof(float) },		// EMaterialParameterType::Vector
				{ (int8)EValueType::Double4, 4, sizeof(double) },	// EMaterialParameterType::DoubleVector
			};
			const int8* ParameterLookup = GParameterCopyLookup[(int32)Parameter.ParameterType];
			int32 NumComponents = ParameterLookup[1];
			int32 ComponentSizeInBytes = ParameterLookup[2];

			FValueComponent* Components = Stack.PushEmptyValue((EValueType)ParameterLookup[0], NumComponents);
			const uint8* ParameterData = UniformExpressionSet->GetDefaultParameterData(Parameter.DefaultValueOffset);

			// Would be nice to be able to do hard coded copies instead of memcpy per component, but it's possible
			// the source data may not be aligned.
			for (int32 ComponentIndex = 0; ComponentIndex < NumComponents; ComponentIndex++)
			{
				FMemory::Memcpy(&Components[ComponentIndex].Packed, ParameterData, ComponentSizeInBytes);
				ParameterData += ComponentSizeInBytes;
			}
		}
		else
		{
			Stack.PushValue(UniformExpressionSet->GetDefaultParameterValue(Parameter.ParameterType, Parameter.DefaultValueOffset));
		}
	}
}

static void EvaluateParameter(FPreshaderDebugStack& Stack, const FUniformExpressionSet* UniformExpressionSet, uint32 ParameterIndex, const FMaterialRenderContext& Context)
{
	Stack.OpCount++;
	Stack.ParameterCount++;

	if (!Stack.bGenerateString && !Stack.ParameterReferences)
	{
		return;
	}

	const FMaterialNumericParameterInfo& Parameter = UniformExpressionSet->GetNumericParameter(ParameterIndex);
	FString ParameterName = Parameter.ParameterInfo.Name.ToString();

	// Count the number of references to each parameter
	if (Stack.ParameterReferences)
	{
		uint32 ParameterNameHash = GetTypeHash(ParameterName);
		uint32* ParameterCount = Stack.ParameterReferences->FindByHash(ParameterNameHash, ParameterName);
		if (ParameterCount)
		{
			(*ParameterCount)++;
		}
		else
		{
			Stack.ParameterReferences->AddByHash(ParameterNameHash, ParameterName, 1);
		}
	}

	if (Stack.bGenerateString)
	{
		Stack.Push(FString::Printf(TEXT("Param[\"%s\"]"), *ParameterName), EPreshaderDebugStackType::Singular);
	}
}

// Utility function to normalize a vector in place, for use with EvaluateUnaryOpInPlace
EValueType NormalizeInPlace(EValueType Type, TArrayView<FValueComponent> Component)
{
	const FValueTypeDescription& TypeDesc = GetValueTypeDescription(Type);
	if (TypeDesc.ComponentType == EValueComponentType::Double)
	{
		// Compute magnitude squared (via dot product)
		double MagnitudeSquared = 0.0;
		for (int32 ComponentIndex = 0; ComponentIndex < TypeDesc.NumComponents; ComponentIndex++)
		{
			double Value = Component[ComponentIndex].Double;
			MagnitudeSquared += Value * Value;
		}

		// Make the magnitude safe to invert.  Similar to logic in GetSafeDivisor, which clamps a value
		// to UE_DELTA, but in this case we are using the delta squared to produce the same result we
		// would get if we did a square root before dividing.
		MagnitudeSquared = FMath::Max(MagnitudeSquared, UE_DOUBLE_DELTA * UE_DOUBLE_DELTA);

		double InverseMagnitude = FMath::InvSqrt(MagnitudeSquared);

		// And apply that back to the components
		for (int32 ComponentIndex = 0; ComponentIndex < TypeDesc.NumComponents; ComponentIndex++)
		{
			Component[ComponentIndex].Double *= InverseMagnitude;
		}

		// Type didn't change, started as Double, still Double
		return Type;
	}
	else
	{
		// Convert the type to float, if it's not already.
		if (TypeDesc.ComponentType != EValueComponentType::Float)
		{
			if (TypeDesc.ComponentType == EValueComponentType::Int)
			{
				for (int32 ComponentIndex = 0; ComponentIndex < TypeDesc.NumComponents; ComponentIndex++)
				{
					Component[ComponentIndex].Float = (float)Component[ComponentIndex].Int;
				}

				// Translate type.  Note that int doesn't support matrices, so we know it should be [Int1..Int4].
				check(Type >= EValueType::Int1 && Type <= EValueType::Int4);
				Type = (EValueType)((int32)Type - (int32)EValueType::Int1 + (int32)EValueType::Float1);
			}
			else if (TypeDesc.ComponentType == EValueComponentType::Bool)
			{
				for (int32 ComponentIndex = 0; ComponentIndex < TypeDesc.NumComponents; ComponentIndex++)
				{
					Component[ComponentIndex].Float = (float)Component[ComponentIndex].Bool;
				}

				// Translate type.  Note that bool doesn't support matrices, so we know it should be [Bool1..Bool4].
				check(Type >= EValueType::Bool1 && Type <= EValueType::Bool4);
				Type = (EValueType)((int32)Type - (int32)EValueType::Bool1 + (int32)EValueType::Float1);
			}
			else
			{
				// Anything else, we'll just bail and not modify the input.  It's probably a bug for the bytecode to
				// generate a normalize operation on a non numeric type.
				return Type;
			}
		}

		// Compute magnitude squared (via dot product)
		float MagnitudeSquared = 0.0;
		for (int32 ComponentIndex = 0; ComponentIndex < TypeDesc.NumComponents; ComponentIndex++)
		{
			float Value = Component[ComponentIndex].Float;
			MagnitudeSquared += Value * Value;
		}

		// Make the magnitude safe to invert.  Similar to logic in GetSafeDivisor, which clamps a value
		// to UE_DELTA, but in this case we are using the delta squared to produce the same result we
		// would get if we did a square root before dividing.
		MagnitudeSquared = FMath::Max(MagnitudeSquared, UE_DELTA * UE_DELTA);

		float InverseMagnitude = FMath::InvSqrt(MagnitudeSquared);

		// And apply that back to the components
		for (int32 ComponentIndex = 0; ComponentIndex < TypeDesc.NumComponents; ComponentIndex++)
		{
			Component[ComponentIndex].Float *= InverseMagnitude;
		}

		// We updated the type to a Float variation above, if it wasn't already Float, so return that
		return Type;
	}
}

template<typename Operation>
static inline void EvaluateUnaryOp(FPreshaderStack& Stack, const Operation& Op)
{
	const FValue Value = Stack.PopValue().AsShaderValue();
	Stack.PushValue(Op(Value));
}

static inline void EvaluateUnaryOpDebug(FPreshaderDebugStack& Stack, const TCHAR* Op)
{
	Stack.OpCount++;

	if (Stack.bGenerateString)
	{
		// Length op is in a lambda, rather than a standalone function, so replace that with "Length" if we detect that's the Op passed in
		Stack.Elements.Last().Text = FString::Printf(TEXT("%s( %s )"), FCString::Strstr(Op, TEXT("Sqrt(Dot(")) ? TEXT("Length") : Op, *Stack.Elements.Last().Text);
		Stack.Elements.Last().DebugStackType = EPreshaderDebugStackType::Singular;
	}
}

template<typename Operation>
static inline void EvaluateUnaryOpInPlace(FPreshaderStack& Stack, const Operation& Op)
{
	// The original EvaluateUnaryOp pops a value from the top of the stack, applies an operation to it, then pushes a new value back
	// on the stack.  This involves moving a lot of memory around, as well as costly bookkeeping.  What we do here instead is keep
	// the top value on the stack, and modify it in place.  We use different variations of shader operations which can be applied
	// to a loose EValueType and array of FValueComponent, as opposed to requiring those two things to be copied into an FValue first.
	// It's possible for any given operation to modify the type, so we override the type of the top item based on the return value.
	FPreshaderValue Top = Stack.PeekValue();
	Stack.OverrideTopType(Op(Top.Type.ValueType, Top.Component));
}

static inline void EvaluateUnaryOpInPlaceDebug(FPreshaderDebugStack& Stack, const TCHAR* Op)
{
	Stack.OpCount++;

	if (Stack.bGenerateString)
	{
		FString LocalOp = Op;
		LocalOp.RemoveFromEnd("InPlace");
		Stack.Elements.Last().Text = FString::Printf(TEXT("%s( %s )"), *LocalOp, *Stack.Elements.Last().Text);
		Stack.Elements.Last().DebugStackType = EPreshaderDebugStackType::Singular;
	}
}

template<typename Operation>
static inline void EvaluateBinaryOp(FPreshaderStack& Stack, const Operation& Op)
{
	const FValue Value1 = Stack.PopValue().AsShaderValue();
	const FValue Value0 = Stack.PopValue().AsShaderValue();
	Stack.PushValue(Op(Value0, Value1));
}

static inline void EvaluateBinaryOpDebug(FPreshaderDebugStack& Stack, const TCHAR* Op)
{
	Stack.OpCount++;

	if (Stack.bGenerateString)
	{
		FPreshaderDebugStackElement Value1 = Stack.Elements.Pop();
		FPreshaderDebugStackElement Value0 = Stack.Elements.Pop();

		EPreshaderOperatorType PreshaderOperator = EPreshaderOperatorType::None;
		if (!FCString::Strcmp(Op, TEXT("Add")))
		{
			PreshaderOperator = EPreshaderOperatorType::Add;
		}
		else if (!FCString::Strcmp(Op, TEXT("Sub")))
		{
			PreshaderOperator = EPreshaderOperatorType::Sub;
		}
		else if (!FCString::Strcmp(Op, TEXT("Mul")))
		{
			PreshaderOperator = EPreshaderOperatorType::Mul;
		}
		else if (!FCString::Strcmp(Op, TEXT("Div")))
		{
			PreshaderOperator = EPreshaderOperatorType::Div;
		}

		if (PreshaderOperator != EPreshaderOperatorType::None)
		{
			static const TCHAR PreshaderOperatorChars[4] = { '+', '-', '*', '/' };

			TCHAR OperatorChar = PreshaderOperatorChars[(uint8)PreshaderOperator];
			EPreshaderDebugStackType NewDebugStackType = PreshaderOperator >= EPreshaderOperatorType::Mul ? EPreshaderDebugStackType::ProductTerms : EPreshaderDebugStackType::SumTerms;
			EPreshaderOperatorType MaxOperatorValid = PreshaderDebugStackLookup[(uint8)Value0.DebugStackType][(uint8)Value1.DebugStackType];

			if (PreshaderOperator <= MaxOperatorValid)
			{
				// Operator is valid without parentheses
				Stack.Push(FString::Printf(TEXT("%s %c %s"), *Value0.Text, OperatorChar, *Value1.Text), NewDebugStackType);
			}
			else
			{
				// Add parentheses around non-singular parameters.  At least one is non-singular, otherwise we would have gone through the above path
				if (Value0.DebugStackType == EPreshaderDebugStackType::Singular)
				{
					Stack.Push(FString::Printf(TEXT("%s %c (%s)"), *Value0.Text, OperatorChar, *Value1.Text), NewDebugStackType);
				}
				else if (Value1.DebugStackType == EPreshaderDebugStackType::Singular)
				{
					Stack.Push(FString::Printf(TEXT("(%s) %c %s"), *Value0.Text, OperatorChar, *Value1.Text), NewDebugStackType);
				}
				else
				{
					Stack.Push(FString::Printf(TEXT("(%s) %c (%s)"), *Value0.Text, OperatorChar, *Value1.Text), NewDebugStackType);
				}
			}
		}
		else
		{
			Stack.Push(FString::Printf(TEXT("%s(%s, %s)"), Op, *Value0.Text, *Value1.Text), EPreshaderDebugStackType::Singular);
		}
	}
}

template<typename Operation, typename OperationFallback>
static inline void EvaluateBinaryOpInPlace(FPreshaderStack& Stack, const Operation& Op, const OperationFallback& FallbackOp)
{
	// Lhs is one below top of stack, Rhs is top
	FPreshaderValue Lhs = Stack.PeekValue(1);
	FPreshaderValue Rhs = Stack.PeekValue(0);

	// Initialize to avoid static analysis warning -- all code paths do fill in ComponentsConsumed
	int32 ComponentsConsumed = 0;
	EValueType ResultType = Op(Lhs.Type.ValueType, Rhs.Type.ValueType, TArrayView<FValueComponent>(Lhs.Component.GetData(), Lhs.Component.Num() + Rhs.Component.Num()), ComponentsConsumed);
	Stack.MergeTopTwoValues(ResultType, ComponentsConsumed);
}

template<typename Operation>
static inline void EvaluateTernaryOp(FPreshaderStack& Stack, const Operation& Op)
{
	const FValue Value2 = Stack.PopValue().AsShaderValue();
	const FValue Value1 = Stack.PopValue().AsShaderValue();
	const FValue Value0 = Stack.PopValue().AsShaderValue();
	Stack.PushValue(Op(Value0, Value1, Value2));
}

static inline void EvaluateTernaryOpDebug(FPreshaderDebugStack& Stack, const TCHAR* Op)
{
	Stack.OpCount++;

	if (Stack.bGenerateString)
	{
		FString Value2 = Stack.Elements.Pop().Text;
		FString Value1 = Stack.Elements.Pop().Text;
		FString Value0 = Stack.Elements.Pop().Text;
		Stack.Push(FString::Printf(TEXT("%s(%s, %s, %s)"), Op, *Value0, *Value1, *Value2), EPreshaderDebugStackType::Singular);
	}
}

// Runs the swizzle in place on the top item on the stack
static void EvaluateComponentSwizzle(FPreshaderStack& Stack, FPreshaderDataContext& RESTRICT Data)
{
	// Get original type and adjust the count and type to the new type.  Adjusting the count doesn't destroy
	// the elements, so we can still access them, but saves time over having to re-fetch the component pointer
	// in case the component array gets resized between the read and write.
	const FPreshaderType& OriginalType = Stack.PeekType();

	const int32 OriginalNumComponents = GetValueTypeDescription(OriginalType.ValueType).NumComponents;
	const int32 NumElements = (int32)ReadPreshaderValue<uint8>(Data);
	
	auto ReadNextSourceComponentIndex = [&Data, OriginalNumComponents]()
	{
		uint8 SourceComponentIndex = ReadPreshaderValue<uint8>(Data);
		if (OriginalNumComponents == 1 && SourceComponentIndex != uint8(INDEX_NONE))
		{
			// Replicate scalar
			SourceComponentIndex = 0;
		}
		return (int32)SourceComponentIndex;
	};
	const int32 IndexR = ReadNextSourceComponentIndex();
	const int32 IndexG = ReadNextSourceComponentIndex();
	const int32 IndexB = ReadNextSourceComponentIndex();
	const int32 IndexA = ReadNextSourceComponentIndex();

	Stack.AdjustComponentCount(NumElements - OriginalNumComponents);
	Stack.OverrideTopType(UE::Shader::MakeValueType(OriginalType.ValueType, NumElements));

	// We've already adjusted the component count for the stack top to the new count, so that's the offset we
	// need to use to get the Component pointer.
	FValueComponent* Components = Stack.PeekComponents(NumElements);

	switch (NumElements)
	{
	case 4:
		{
			uint64 PackedR = IndexR < OriginalNumComponents ? Components[IndexR].Packed : 0;
			uint64 PackedG = IndexG < OriginalNumComponents ? Components[IndexG].Packed : 0;
			uint64 PackedB = IndexB < OriginalNumComponents ? Components[IndexB].Packed : 0;
			uint64 PackedA = IndexA < OriginalNumComponents ? Components[IndexA].Packed : 0;
			Components[0].Packed = PackedR;
			Components[1].Packed = PackedG;
			Components[2].Packed = PackedB;
			Components[3].Packed = PackedA;
		}
		break;
	case 3:
		{
			uint64 PackedR = IndexR < OriginalNumComponents ? Components[IndexR].Packed : 0;
			uint64 PackedG = IndexG < OriginalNumComponents ? Components[IndexG].Packed : 0;
			uint64 PackedB = IndexB < OriginalNumComponents ? Components[IndexB].Packed : 0;
			Components[0].Packed = PackedR;
			Components[1].Packed = PackedG;
			Components[2].Packed = PackedB;
		}
		break;
	case 2:
		{
			uint64 PackedR = IndexR < OriginalNumComponents ? Components[IndexR].Packed : 0;
			uint64 PackedG = IndexG < OriginalNumComponents ? Components[IndexG].Packed : 0;
			Components[0].Packed = PackedR;
			Components[1].Packed = PackedG;
		}
		break;
	case 1:
		{
			uint64 PackedR = IndexR < OriginalNumComponents ? Components[IndexR].Packed : 0;
			Components[0].Packed = PackedR;
		}
		break;
	default:
		UE_LOGF(LogMaterial, Fatal, "Invalid number of swizzle elements: %d", NumElements);
		break;
	}
}

static void EvaluateComponentSwizzle(FPreshaderDebugStack& Stack, FPreshaderDataContext& RESTRICT Data)
{
	Stack.OpCount++;

	const uint8 NumElements = ReadPreshaderValue<uint8>(Data);
	const uint8 IndexR = FMath::Min(ReadPreshaderValue<uint8>(Data), (uint8)3);
	const uint8 IndexG = FMath::Min(ReadPreshaderValue<uint8>(Data), (uint8)3);
	const uint8 IndexB = FMath::Min(ReadPreshaderValue<uint8>(Data), (uint8)3);
	const uint8 IndexA = FMath::Min(ReadPreshaderValue<uint8>(Data), (uint8)3);

	if (Stack.bGenerateString)
	{
		if (Stack.Elements.Last().DebugStackType != EPreshaderDebugStackType::Singular)
		{
			Stack.Elements.Last().Text = FString::Printf(TEXT("(%s)"), *Stack.Elements.Last().Text);
			Stack.Elements.Last().DebugStackType = EPreshaderDebugStackType::Singular;
		}

		static const TCHAR ComponentChars[4] = { 'x', 'y', 'z', 'w' };

		switch (NumElements)
		{
		case 4:
			Stack.Elements.Last().Text = FString::Printf(TEXT("%s.%c%c%c%c"), *Stack.Elements.Last().Text, ComponentChars[IndexR], ComponentChars[IndexG], ComponentChars[IndexB], ComponentChars[IndexA]);
			break;
		case 3:
			Stack.Elements.Last().Text = FString::Printf(TEXT("%s.%c%c%c"), *Stack.Elements.Last().Text, ComponentChars[IndexR], ComponentChars[IndexG], ComponentChars[IndexB]);
			break;
		case 2:
			Stack.Elements.Last().Text = FString::Printf(TEXT("%s.%c%c"), *Stack.Elements.Last().Text, ComponentChars[IndexR], ComponentChars[IndexG]);
			break;
		case 1:
			Stack.Elements.Last().Text = FString::Printf(TEXT("%s.%c"), *Stack.Elements.Last().Text, ComponentChars[IndexR]);
			break;
		}
	}
}

static void GetTextureParameter(const FMaterialRenderContext& Context, FPreshaderDataContext& RESTRICT Data, const UTexture*& OutTexture, const USparseVolumeTexture*& OutSparseVolumeTexture)
{
	const FHashedMaterialParameterInfo ParameterInfo = ReadPreshaderValue<FHashedMaterialParameterInfo>(Data);
	const int32 TextureIndex = ReadPreshaderValue<int32>(Data);

	OutTexture = nullptr;
	OutSparseVolumeTexture = nullptr;

	Context.GetTextureParameterValue(ParameterInfo, TextureIndex, OutTexture);
	if (!OutTexture)
	{
		Context.GetTextureParameterValue(ParameterInfo, TextureIndex, OutSparseVolumeTexture);
	}
}

static void EvaluateTextureSize(const FMaterialRenderContext& Context, FPreshaderStack& Stack, FPreshaderDataContext& RESTRICT Data)
{
	const UTexture* Texture = nullptr;
	const USparseVolumeTexture* SparseVolumeTexture = nullptr;
	GetTextureParameter(Context, Data, Texture, SparseVolumeTexture);
	if (Texture && Texture->GetResource())
	{
		const uint32 SizeX = Texture->GetResource()->GetSizeX();
		const uint32 SizeY = Texture->GetResource()->GetSizeY();
		const uint32 SizeZ = Texture->GetResource()->GetSizeZ();
		Stack.PushValue(FValue((float)SizeX, (float)SizeY, (float)SizeZ));
	}
	else if (SparseVolumeTexture)
	{
		const uint32 SizeX = SparseVolumeTexture->GetSizeX();
		const uint32 SizeY = SparseVolumeTexture->GetSizeY();
		const uint32 SizeZ = SparseVolumeTexture->GetSizeZ();
		Stack.PushValue(FValue((float)SizeX, (float)SizeY, (float)SizeZ));
	}
	else
	{
		Stack.PushValue(FValue(0.0f, 0.0f, 0.0f));
	}
}

static void EvaluateTextureSize(const FMaterialRenderContext& Context, FPreshaderDebugStack& Stack, FPreshaderDataContext& RESTRICT Data)
{
	Stack.OpCount++;

	const FHashedMaterialParameterInfo ParameterInfo = ReadPreshaderValue<FHashedMaterialParameterInfo>(Data);
	const int32 TextureIndex = ReadPreshaderValue<int32>(Data);
	if (Stack.bGenerateString)
	{
		Stack.Push(FString::Printf(TEXT("TEX[\"%s\"][%d].TextureSize"), *ParameterInfo.GetName().ToString(), TextureIndex), EPreshaderDebugStackType::Singular);
	}
}

static void EvaluateTexelSize(const FMaterialRenderContext& Context, FPreshaderStack& Stack, FPreshaderDataContext& RESTRICT Data)
{
	const UTexture* Texture = nullptr;
	const USparseVolumeTexture* SparseVolumeTexture = nullptr;
	GetTextureParameter(Context, Data, Texture, SparseVolumeTexture);
	if (Texture && Texture->GetResource())
	{
		const uint32 SizeX = Texture->GetResource()->GetSizeX();
		const uint32 SizeY = Texture->GetResource()->GetSizeY();
		const uint32 SizeZ = Texture->GetResource()->GetSizeZ();
		Stack.PushValue(FValue(1.0f / (float)SizeX, 1.0f / (float)SizeY, (SizeZ > 0 ? 1.0f / (float)SizeZ : 0.0f)));
	}
	else if (SparseVolumeTexture)
	{
		const uint32 SizeX = SparseVolumeTexture->GetSizeX();
		const uint32 SizeY = SparseVolumeTexture->GetSizeY();
		const uint32 SizeZ = SparseVolumeTexture->GetSizeZ();
		Stack.PushValue(FValue(1.0f / (float)SizeX, 1.0f / (float)SizeY, 1.0f / (float)SizeZ));
	}
	else
	{
		Stack.PushValue(FValue(0.0f, 0.0f, 0.0f));
	}
}

static void EvaluateTexelSize(const FMaterialRenderContext& Context, FPreshaderDebugStack& Stack, FPreshaderDataContext& RESTRICT Data)
{
	Stack.OpCount++;

	const FHashedMaterialParameterInfo ParameterInfo = ReadPreshaderValue<FHashedMaterialParameterInfo>(Data);
	const int32 TextureIndex = ReadPreshaderValue<int32>(Data);
	if (Stack.bGenerateString)
	{
		Stack.Push(FString::Printf(TEXT("TEX[\"%s\"][%d].TexelSize"), *ParameterInfo.GetName().ToString(), TextureIndex), EPreshaderDebugStackType::Singular);
	}
}

static FGuid GetExternalTextureGuid(const FMaterialRenderContext& Context, FPreshaderDataContext& RESTRICT Data)
{
	const FScriptName ParameterName = ReadPreshaderValue<FScriptName>(Data);
	const FGuid ExternalTextureGuid = ReadPreshaderValue<FGuid>(Data);
	const int32 TextureIndex = ReadPreshaderValue<int32>(Data);
	return Context.GetExternalTextureGuid(ExternalTextureGuid, ScriptNameToName(ParameterName), TextureIndex);
}

static void EvaluateExternalTextureCoordinateScaleRotation(const FMaterialRenderContext& Context, FPreshaderStack& Stack, FPreshaderDataContext& RESTRICT Data)
{
	const FGuid GuidToLookup = GetExternalTextureGuid(Context, Data);
	FLinearColor Result(1.f, 0.f, 0.f, 1.f);
	if (GuidToLookup.IsValid())
	{
		FExternalTextureRegistry::Get().GetExternalTextureCoordinateScaleRotation(GuidToLookup, Result);
	}
	Stack.PushValue(Result);
}

static void EvaluateExternalTextureCoordinateOffset(const FMaterialRenderContext& Context, FPreshaderStack& Stack, FPreshaderDataContext& RESTRICT Data)
{
	const FGuid GuidToLookup = GetExternalTextureGuid(Context, Data);
	FLinearColor Result(0.f, 0.f, 0.f, 0.f);
	if (GuidToLookup.IsValid())
	{
		FExternalTextureRegistry::Get().GetExternalTextureCoordinateOffset(GuidToLookup, Result);
	}
	Stack.PushValue(Result);
}

static void EvaluateExternalTextureCoordinateScaleRotation(const FMaterialRenderContext& Context, FPreshaderDebugStack& Stack, FPreshaderDataContext& RESTRICT Data)
{
	Stack.OpCount++;

	const FScriptName ParameterName = ReadPreshaderValue<FScriptName>(Data);
	const FGuid ExternalTextureGuid = ReadPreshaderValue<FGuid>(Data);
	const int32 TextureIndex = ReadPreshaderValue<int32>(Data);

	if (Stack.bGenerateString)
	{
		Stack.Push(FString::Printf(TEXT("TEX[\"%s\"].ExternalTextureScaleRotation"), *ParameterName.ToString()), EPreshaderDebugStackType::Singular);
	}
}

static void EvaluateExternalTextureCoordinateOffset(const FMaterialRenderContext& Context, FPreshaderDebugStack& Stack, FPreshaderDataContext& RESTRICT Data)
{
	Stack.OpCount++;

	const FScriptName ParameterName = ReadPreshaderValue<FScriptName>(Data);
	const FGuid ExternalTextureGuid = ReadPreshaderValue<FGuid>(Data);
	const int32 TextureIndex = ReadPreshaderValue<int32>(Data);

	if (Stack.bGenerateString)
	{
		Stack.Push(FString::Printf(TEXT("TEX[\"%s\"].ExternalTextureOffset"), *ParameterName.ToString()), EPreshaderDebugStackType::Singular);
	}
}

static void EvaluateRuntimeVirtualTextureUniform(const FMaterialRenderContext& Context, FPreshaderStack& Stack, FPreshaderDataContext& RESTRICT Data)
{
	const FHashedMaterialParameterInfo ParameterInfo = ReadPreshaderValue<FHashedMaterialParameterInfo>(Data);
	const int32 TextureIndex = ReadPreshaderValue<int32>(Data);
	const int32 VectorIndex = ReadPreshaderValue<int32>(Data);

	const URuntimeVirtualTexture* Texture = nullptr;
	if (ParameterInfo.Name.IsNone() || !Context.MaterialRenderProxy || !Context.MaterialRenderProxy->GetTextureValue(ParameterInfo, &Texture, Context))
	{
		Texture = GetIndexedTexture<URuntimeVirtualTexture>(Context.Material, TextureIndex);
	}
	if (Texture != nullptr && VectorIndex != INDEX_NONE)
	{
		Stack.PushValue(FValue(Texture->GetUniformParameter(VectorIndex)));
	}
	else
	{
		Stack.PushValue(FValue(0.f, 0.f, 0.f, 0.f));
	}
}

static void EvaluateRuntimeVirtualTextureUniform(const FMaterialRenderContext& Context, FPreshaderDebugStack& Stack, FPreshaderDataContext& RESTRICT Data)
{
	Stack.OpCount++;

	const FHashedMaterialParameterInfo ParameterInfo = ReadPreshaderValue<FHashedMaterialParameterInfo>(Data);
	const int32 TextureIndex = ReadPreshaderValue<int32>(Data);
	const int32 VectorIndex = ReadPreshaderValue<int32>(Data);

	if (Stack.bGenerateString)
	{
		Stack.Push(FString::Printf(TEXT("TEX[\"%s\"].VirtualTextureUniform[%d]"), *ParameterInfo.Name.ToString(), VectorIndex), EPreshaderDebugStackType::Singular);
	}
}

static void EvaluateSparseTextureUniform(const FMaterialRenderContext& Context, FPreshaderStack& Stack, FPreshaderDataContext& RESTRICT Data)
{
	const FHashedMaterialParameterInfo ParameterInfo = ReadPreshaderValue<FHashedMaterialParameterInfo>(Data);
	const int32 TextureIndex = ReadPreshaderValue<int32>(Data);
	const int32 VectorIndex = ReadPreshaderValue<int32>(Data);

	const USparseVolumeTexture* Texture = nullptr;
	if (ParameterInfo.Name.IsNone() || !Context.MaterialRenderProxy || !Context.MaterialRenderProxy->GetTextureValue(ParameterInfo, &Texture, Context))
	{
		Texture = GetIndexedTexture<USparseVolumeTexture>(Context.Material, TextureIndex);
	}
	if (Texture != nullptr && VectorIndex != INDEX_NONE)
	{
		Stack.PushValue(FValue(Texture->GetUniformParameter(VectorIndex)));
	}
	else
	{
		Stack.PushValue(FValue(0.f, 0.f, 0.f, 0.f));
	}
}

static void EvaluateSparseTextureUniform(const FMaterialRenderContext& Context, FPreshaderDebugStack& Stack, FPreshaderDataContext& RESTRICT Data)
{
	Stack.OpCount++;

	const FHashedMaterialParameterInfo ParameterInfo = ReadPreshaderValue<FHashedMaterialParameterInfo>(Data);
	const int32 TextureIndex = ReadPreshaderValue<int32>(Data);
	const int32 VectorIndex = ReadPreshaderValue<int32>(Data);

	if (Stack.bGenerateString)
	{
		Stack.Push(FString::Printf(TEXT("TEX[\"%s\"].SparseTextureUniform[%d]"), *ParameterInfo.Name.ToString(), VectorIndex), EPreshaderDebugStackType::Singular);
	}
}

FPreshaderValue EvaluatePreshader(const FUniformExpressionSet* UniformExpressionSet, const FMaterialRenderContext& Context, FPreshaderStack& Stack, FPreshaderDataContext& RESTRICT Data)
{
	uint8 const* const DataEnd = Data.EndPtr;

	Stack.Reset();
	while (Data.Ptr < DataEnd)
	{
		const EPreshaderOpcode Opcode = (EPreshaderOpcode)ReadPreshaderValue<uint8>(Data);
		switch (Opcode)
		{
		case EPreshaderOpcode::ConstantZero: EvaluateConstantZero(Stack, Data); break;
		case EPreshaderOpcode::Constant: EvaluateConstant(Stack, Data); break;
		case EPreshaderOpcode::Parameter:
			EvaluateParameter(Stack, UniformExpressionSet, ReadPreshaderValue<uint16>(Data), Context);
			break;
		case EPreshaderOpcode::PushValue: EvaluatePushValue(Stack, Data); break;
		case EPreshaderOpcode::Assign: EvaluateAssign(Stack); break;
		case EPreshaderOpcode::Add: EvaluateBinaryOpInPlace(Stack, AddInPlace, Add); break;
		case EPreshaderOpcode::Sub: EvaluateBinaryOpInPlace(Stack, SubInPlace, Sub); break;
		case EPreshaderOpcode::Mul: EvaluateBinaryOpInPlace(Stack, MulInPlace, Mul); break;
		case EPreshaderOpcode::Div: EvaluateBinaryOpInPlace(Stack, DivInPlace, Div); break;
		case EPreshaderOpcode::Less: EvaluateBinaryOp(Stack, Less); break;
		case EPreshaderOpcode::Greater: EvaluateBinaryOp(Stack, Greater); break;
		case EPreshaderOpcode::LessEqual: EvaluateBinaryOp(Stack, LessEqual); break;
		case EPreshaderOpcode::GreaterEqual: EvaluateBinaryOp(Stack, GreaterEqual); break;
		case EPreshaderOpcode::Fmod: EvaluateBinaryOpInPlace(Stack, FmodInPlace, Fmod); break;
		case EPreshaderOpcode::Modulo: EvaluateBinaryOpInPlace(Stack, ModuloInPlace, Modulo); break;
		case EPreshaderOpcode::Min: EvaluateBinaryOpInPlace(Stack, MinInPlace, Min); break;
		case EPreshaderOpcode::Max: EvaluateBinaryOpInPlace(Stack, MaxInPlace, Max); break;
		case EPreshaderOpcode::Clamp: EvaluateTernaryOp(Stack, Clamp); break;
		case EPreshaderOpcode::Dot: EvaluateBinaryOp(Stack, Dot); break;
		case EPreshaderOpcode::Cross: EvaluateBinaryOp(Stack, Cross); break;
		case EPreshaderOpcode::Neg: EvaluateUnaryOpInPlace(Stack, NegInPlace); break;
		case EPreshaderOpcode::Sqrt: EvaluateUnaryOpInPlace(Stack, SqrtInPlace); break;
		case EPreshaderOpcode::Rcp: EvaluateUnaryOpInPlace(Stack, RcpInPlace); break;
		case EPreshaderOpcode::Length: EvaluateUnaryOp(Stack, [](const FValue& Value) { return Sqrt(Dot(Value, Value)); }); break;
		case EPreshaderOpcode::Normalize: EvaluateUnaryOpInPlace(Stack, NormalizeInPlace); break;
		case EPreshaderOpcode::Sin: EvaluateUnaryOpInPlace(Stack, SinInPlace); break;
		case EPreshaderOpcode::Cos: EvaluateUnaryOpInPlace(Stack, CosInPlace); break;
		case EPreshaderOpcode::Tan: EvaluateUnaryOpInPlace(Stack, TanInPlace); break;
		case EPreshaderOpcode::Asin: EvaluateUnaryOpInPlace(Stack, AsinInPlace); break;;
		case EPreshaderOpcode::Acos: EvaluateUnaryOpInPlace(Stack, AcosInPlace); break;
		case EPreshaderOpcode::Atan: EvaluateUnaryOpInPlace(Stack, AtanInPlace); break;
		case EPreshaderOpcode::Atan2: EvaluateBinaryOpInPlace(Stack, Atan2InPlace, Atan2); break;
		case EPreshaderOpcode::Abs: EvaluateUnaryOpInPlace(Stack, AbsInPlace); break;
		case EPreshaderOpcode::Saturate: EvaluateUnaryOpInPlace(Stack, SaturateInPlace); break;
		case EPreshaderOpcode::Floor: EvaluateUnaryOpInPlace(Stack, FloorInPlace); break;
		case EPreshaderOpcode::Ceil: EvaluateUnaryOpInPlace(Stack, CeilInPlace); break;
		case EPreshaderOpcode::Round: EvaluateUnaryOpInPlace(Stack, RoundInPlace); break;
		case EPreshaderOpcode::Trunc: EvaluateUnaryOpInPlace(Stack, TruncInPlace); break;
		case EPreshaderOpcode::Sign: EvaluateUnaryOpInPlace(Stack, SignInPlace); break;
		case EPreshaderOpcode::Frac: EvaluateUnaryOpInPlace(Stack, FracInPlace); break;
		case EPreshaderOpcode::Fractional: EvaluateUnaryOpInPlace(Stack, FractionalInPlace); break;
		case EPreshaderOpcode::Exp: EvaluateUnaryOpInPlace(Stack, ExpInPlace); break;
		case EPreshaderOpcode::Exp2: EvaluateUnaryOpInPlace(Stack, Exp2InPlace); break;
		case EPreshaderOpcode::Log: EvaluateUnaryOpInPlace(Stack, LogInPlace); break;
		case EPreshaderOpcode::Log2: EvaluateUnaryOpInPlace(Stack, Log2InPlace); break;
		case EPreshaderOpcode::Log10: EvaluateUnaryOpInPlace(Stack, Log10InPlace); break;
		case EPreshaderOpcode::ComponentSwizzle: EvaluateComponentSwizzle(Stack, Data); break;
		case EPreshaderOpcode::AppendVector: EvaluateBinaryOpInPlace(Stack, AppendInPlace, Append); break;
		case EPreshaderOpcode::TextureSize: EvaluateTextureSize(Context, Stack, Data); break;
		case EPreshaderOpcode::TexelSize: EvaluateTexelSize(Context, Stack, Data); break;
		case EPreshaderOpcode::ExternalTextureCoordinateScaleRotation: EvaluateExternalTextureCoordinateScaleRotation(Context, Stack, Data); break;
		case EPreshaderOpcode::ExternalTextureCoordinateOffset: EvaluateExternalTextureCoordinateOffset(Context, Stack, Data); break;
		case EPreshaderOpcode::RuntimeVirtualTextureUniform: EvaluateRuntimeVirtualTextureUniform(Context, Stack, Data); break;
		case EPreshaderOpcode::SparseVolumeTextureUniform: EvaluateSparseTextureUniform(Context, Stack, Data); break;
		default:
			UE_LOGF(LogMaterial, Fatal, "Unknown preshader opcode %d", (uint8)Opcode);
			break;
		}
	}
	check(Data.Ptr == DataEnd);

	FPreshaderValue Result;
	Stack.PopResult(Result);
	return Result;
}

#define EvaluateUnaryOp(STACK, OP) EvaluateUnaryOpDebug(STACK, TEXT(#OP))
#define EvaluateUnaryOpInPlace(STACK, IN_PLACE_OP) EvaluateUnaryOpInPlaceDebug(STACK, TEXT(#IN_PLACE_OP))
#define EvaluateBinaryOp(STACK, OP) EvaluateBinaryOpDebug(STACK, TEXT(#OP))
#define EvaluateBinaryOpInPlace(STACK, IN_PLACE_OP, OP) EvaluateBinaryOpDebug(STACK, TEXT(#OP))
#define EvaluateTernaryOp(STACK, OP) EvaluateTernaryOpDebug(STACK, TEXT(#OP))

// The body of this function is designed to be completely identical to EvaluatePreshader above, other than the lack of popping the result
// at the end, so new opcodes can be added and the body trivially cut-and-pasted, for ease of maintenance.  Temporary redefining some Evaluate
// functions as macros is done above to automate string-izing of function names.
static void EvaluatePreshaderDebug(const FUniformExpressionSet* UniformExpressionSet, const FMaterialRenderContext& Context, FPreshaderDebugStack& Stack, FPreshaderDataContext& RESTRICT Data)
{
	uint8 const* const DataEnd = Data.EndPtr;

	Stack.Reset();
	while (Data.Ptr < DataEnd)
	{
		const EPreshaderOpcode Opcode = (EPreshaderOpcode)ReadPreshaderValue<uint8>(Data);
		switch (Opcode)
		{
		case EPreshaderOpcode::ConstantZero: EvaluateConstantZero(Stack, Data); break;
		case EPreshaderOpcode::Constant: EvaluateConstant(Stack, Data); break;
		case EPreshaderOpcode::Parameter:
			EvaluateParameter(Stack, UniformExpressionSet, ReadPreshaderValue<uint16>(Data), Context);
			break;
		case EPreshaderOpcode::PushValue: EvaluatePushValue(Stack, Data); break;
		case EPreshaderOpcode::Assign: EvaluateAssign(Stack); break;
		case EPreshaderOpcode::Add: EvaluateBinaryOpInPlace(Stack, AddInPlace, Add); break;
		case EPreshaderOpcode::Sub: EvaluateBinaryOpInPlace(Stack, SubInPlace, Sub); break;
		case EPreshaderOpcode::Mul: EvaluateBinaryOpInPlace(Stack, MulInPlace, Mul); break;
		case EPreshaderOpcode::Div: EvaluateBinaryOpInPlace(Stack, DivInPlace, Div); break;
		case EPreshaderOpcode::Less: EvaluateBinaryOp(Stack, Less); break;
		case EPreshaderOpcode::Greater: EvaluateBinaryOp(Stack, Greater); break;
		case EPreshaderOpcode::LessEqual: EvaluateBinaryOp(Stack, LessEqual); break;
		case EPreshaderOpcode::GreaterEqual: EvaluateBinaryOp(Stack, GreaterEqual); break;
		case EPreshaderOpcode::Fmod: EvaluateBinaryOpInPlace(Stack, FmodInPlace, Fmod); break;
		case EPreshaderOpcode::Modulo: EvaluateBinaryOpInPlace(Stack, ModuloInPlace, Modulo); break;
		case EPreshaderOpcode::Min: EvaluateBinaryOpInPlace(Stack, MinInPlace, Min); break;
		case EPreshaderOpcode::Max: EvaluateBinaryOpInPlace(Stack, MaxInPlace, Max); break;
		case EPreshaderOpcode::Clamp: EvaluateTernaryOp(Stack, Clamp); break;
		case EPreshaderOpcode::Dot: EvaluateBinaryOp(Stack, Dot); break;
		case EPreshaderOpcode::Cross: EvaluateBinaryOp(Stack, Cross); break;
		case EPreshaderOpcode::Neg: EvaluateUnaryOpInPlace(Stack, NegInPlace); break;
		case EPreshaderOpcode::Sqrt: EvaluateUnaryOpInPlace(Stack, SqrtInPlace); break;
		case EPreshaderOpcode::Rcp: EvaluateUnaryOpInPlace(Stack, RcpInPlace); break;
		case EPreshaderOpcode::Length: EvaluateUnaryOp(Stack, [](const FValue& Value) { return Sqrt(Dot(Value, Value)); }); break;
		case EPreshaderOpcode::Normalize: EvaluateUnaryOpInPlace(Stack, NormalizeInPlace); break;
		case EPreshaderOpcode::Sin: EvaluateUnaryOpInPlace(Stack, SinInPlace); break;
		case EPreshaderOpcode::Cos: EvaluateUnaryOpInPlace(Stack, CosInPlace); break;
		case EPreshaderOpcode::Tan: EvaluateUnaryOpInPlace(Stack, TanInPlace); break;
		case EPreshaderOpcode::Asin: EvaluateUnaryOpInPlace(Stack, AsinInPlace); break;;
		case EPreshaderOpcode::Acos: EvaluateUnaryOpInPlace(Stack, AcosInPlace); break;
		case EPreshaderOpcode::Atan: EvaluateUnaryOpInPlace(Stack, AtanInPlace); break;
		case EPreshaderOpcode::Atan2: EvaluateBinaryOpInPlace(Stack, Atan2InPlace, Atan2); break;
		case EPreshaderOpcode::Abs: EvaluateUnaryOpInPlace(Stack, AbsInPlace); break;
		case EPreshaderOpcode::Saturate: EvaluateUnaryOpInPlace(Stack, SaturateInPlace); break;
		case EPreshaderOpcode::Floor: EvaluateUnaryOpInPlace(Stack, FloorInPlace); break;
		case EPreshaderOpcode::Ceil: EvaluateUnaryOpInPlace(Stack, CeilInPlace); break;
		case EPreshaderOpcode::Round: EvaluateUnaryOpInPlace(Stack, RoundInPlace); break;
		case EPreshaderOpcode::Trunc: EvaluateUnaryOpInPlace(Stack, TruncInPlace); break;
		case EPreshaderOpcode::Sign: EvaluateUnaryOpInPlace(Stack, SignInPlace); break;
		case EPreshaderOpcode::Frac: EvaluateUnaryOpInPlace(Stack, FracInPlace); break;
		case EPreshaderOpcode::Fractional: EvaluateUnaryOpInPlace(Stack, FractionalInPlace); break;
		case EPreshaderOpcode::Exp: EvaluateUnaryOpInPlace(Stack, ExpInPlace); break;
		case EPreshaderOpcode::Exp2: EvaluateUnaryOpInPlace(Stack, Exp2InPlace); break;
		case EPreshaderOpcode::Log: EvaluateUnaryOpInPlace(Stack, LogInPlace); break;
		case EPreshaderOpcode::Log2: EvaluateUnaryOpInPlace(Stack, Log2InPlace); break;
		case EPreshaderOpcode::Log10: EvaluateUnaryOpInPlace(Stack, Log10InPlace); break;
		case EPreshaderOpcode::ComponentSwizzle: EvaluateComponentSwizzle(Stack, Data); break;
		case EPreshaderOpcode::AppendVector: EvaluateBinaryOpInPlace(Stack, AppendInPlace, Append); break;
		case EPreshaderOpcode::TextureSize: EvaluateTextureSize(Context, Stack, Data); break;
		case EPreshaderOpcode::TexelSize: EvaluateTexelSize(Context, Stack, Data); break;
		case EPreshaderOpcode::ExternalTextureCoordinateScaleRotation: EvaluateExternalTextureCoordinateScaleRotation(Context, Stack, Data); break;
		case EPreshaderOpcode::ExternalTextureCoordinateOffset: EvaluateExternalTextureCoordinateOffset(Context, Stack, Data); break;
		case EPreshaderOpcode::RuntimeVirtualTextureUniform: EvaluateRuntimeVirtualTextureUniform(Context, Stack, Data); break;
		case EPreshaderOpcode::SparseVolumeTextureUniform: EvaluateSparseTextureUniform(Context, Stack, Data); break;
		default:
			UE_LOGF(LogMaterial, Fatal, "Unknown preshader opcode %d", (uint8)Opcode);
			break;
		}
	}
	check(Data.Ptr == DataEnd);
}

#undef EvaluateUnaryOp
#undef EvaluateUnaryOpInPlace
#undef EvaluateBinaryOp
#undef EvaluateBinaryOpInPlace
#undef EvaluateTernaryOp

FString PreshaderGenerateDebugString(const FUniformExpressionSet& UniformExpressionSet, const FMaterialRenderContext& Context, FPreshaderDataContext& RESTRICT Data, TMap<FString, uint32>* ParameterReferences)
{
	FPreshaderDebugStack Stack;
	Stack.ParameterReferences = ParameterReferences;
	EvaluatePreshaderDebug(&UniformExpressionSet, Context, Stack, Data);

	return Stack.Elements.Pop().Text;
}

void PreshaderComputeDebugStats(const FUniformExpressionSet& UniformExpressionSet, const FMaterialRenderContext& Context, FPreshaderDataContext& RESTRICT Data, uint32& TotalParameters, uint32& TotalOps)
{
	FPreshaderDebugStack Stack;
	Stack.bGenerateString = false;
	EvaluatePreshaderDebug(&UniformExpressionSet, Context, Stack, Data);

	TotalParameters += Stack.ParameterCount;
	TotalOps += Stack.OpCount;
}

FPreshaderValue FPreshaderData::Evaluate(FUniformExpressionSet* UniformExpressionSet, const struct FMaterialRenderContext& Context, FPreshaderStack& Stack) const
{
	FPreshaderDataContext PreshaderContext(*this);
	return EvaluatePreshader(UniformExpressionSet, Context, Stack, PreshaderContext);
}

FPreshaderValue FPreshaderData::EvaluateConstant(const FMaterial& Material, FPreshaderStack& Stack) const
{
	FPreshaderDataContext PreshaderContext(*this);
	return EvaluatePreshader(nullptr, FMaterialRenderContext(nullptr, Material, nullptr), Stack, PreshaderContext);
}

void FPreshaderData::AppendHash(FXxHash64Builder& OutHasher) const
{
	OutHasher.Update(&bPreshader2, sizeof(bPreshader2));
	OutHasher.Update(Names.GetData(), Names.Num() * Names.GetTypeSize());
	OutHasher.Update(Data.GetData(), Data.Num() * Data.GetTypeSize());
}

} // namespace Shader
} // namespace UE
