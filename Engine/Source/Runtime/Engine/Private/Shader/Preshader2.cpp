// Copyright Epic Games, Inc. All Rights Reserved.

#include "Shader/Preshader2.h"
#include "Shader/PreshaderEvaluate.h"
#include "Materials/MaterialRenderProxy.h"
#include "Materials/MaterialUniformExpressions.h"
#include "Math/DoubleFloat.h"
#include "Misc/LargeWorldRenderPosition.h"
#include "SparseVolumeTexture/SparseVolumeTexture.h"
#include "VT/RuntimeVirtualTexture.h"

namespace UE::Preshader2
{

#define PRESHADER2_UNARY_OP(OP_NAME, ALLOW_INTEGER)	\
	TEXT(#OP_NAME),									\
	TEXT(#OP_NAME "_InPlace"),						\
	TEXT(#OP_NAME "_Reuse"),

#define PRESHADER2_BINARY_OP(OP_NAME, ALLOW_INTEGER)	\
	TEXT(#OP_NAME),										\
	TEXT(#OP_NAME "_InPlace"),							\
	TEXT(#OP_NAME "_Reuse"),							\
	TEXT(#OP_NAME "_Const"),							\
	TEXT(#OP_NAME "_ConstInPlace"),						\
	TEXT(#OP_NAME "_ConstReuse"),						\
	TEXT(#OP_NAME "_ConstReuseScalar"),

#define PRESHADER2_UNARY_OP_CAST_REUSE(OP_NAME) TEXT(#OP_NAME),
#define PRESHADER2_UNARY_OP_CAST_REUSE_LAST(OP_NAME) TEXT(#OP_NAME),
#define PRESHADER2_UNARY_OP_CUSTOM(OP_NAME) TEXT(#OP_NAME),
#define PRESHADER2_BINARY_OP_CUSTOM(OP_NAME) TEXT(#OP_NAME),
#define PRESHADER2_OP(OP_NAME) TEXT(#OP_NAME),

static const TCHAR* GPreshader2OpcodeNames[] =
{
	TEXT("Invalid"),
	#include "Shader/Preshader2Opcodes.inl"
};

#undef PRESHADER2_UNARY_OP
#undef PRESHADER2_BINARY_OP
#undef PRESHADER2_UNARY_OP_CAST_REUSE
#undef PRESHADER2_UNARY_OP_CAST_REUSE_LAST
#undef PRESHADER2_UNARY_OP_CUSTOM
#undef PRESHADER2_BINARY_OP_CUSTOM
#undef PRESHADER2_OP

static const TCHAR* GPreshader2TypeNames[] =
{
	TEXT("float"),		// EPreshader2Type::Float
	TEXT("double"),		// EPreshader2Type::Double
	TEXT("int"),		// EPreshader2Type::Integer,
	TEXT("LWC"),		// EPreshader2Type::LWC
};

static const TCHAR* GPreshader2DimensionNames[] =
{
	TEXT("?"),			// Invalid zero dimension
	TEXT(""),			// Use an empty string for one
	TEXT("2"),
	TEXT("3"),
	TEXT("4"),
	TEXT("5"),			// Fill out the array to avoid static analysis warnings when reading from it,
	TEXT("6"),			// even though valid opcodes won't have dimensions above 4 set.
	TEXT("7"),
};

// Table of swizzles by first component index and component count
static const TCHAR* GPreshader2LinearSwizzles[4][5] =
{
	{ TEXT(".?"), TEXT(".x"), TEXT(".xy"), TEXT(".xyz"), TEXT(".xyzw") },
	{ TEXT(".?"), TEXT(".y"), TEXT(".yz"), TEXT(".yzw"), TEXT(".????") },
	{ TEXT(".?"), TEXT(".z"), TEXT(".zw"), TEXT(".???"), TEXT(".????") },
	{ TEXT(".?"), TEXT(".w"), TEXT(".??"), TEXT(".???"), TEXT(".????") },
};

// Matches GetSafeDivisor in ShaderValue.cpp -- clamps near-zero divisors to avoid inf/NaN.
template<typename T>
FORCENOINLINE static T GetSafeDivisor(T Number)
{
	if (FMath::Abs(Number) < (T)UE_DELTA)
	{
		return (Number < 0) ? -(T)UE_DELTA : (T)UE_DELTA;
	}
	return Number;
}

template<>
FORCENOINLINE int32 GetSafeDivisor<int32>(int32 Number)
{
	return Number != 0 ? Number : 1;
}

struct FOpAbs { template<typename T> static T Call(T Value) { return FMath::Abs(Value); } };
struct FOpACos { template<typename T> static T Call(T Value) { return FMath::Acos(Value); } };
struct FOpASin { template<typename T> static T Call(T Value) { return FMath::Asin(Value); } };
struct FOpATan { template<typename T> static T Call(T Value) { return FMath::Atan(Value); } };
struct FOpCeil { template<typename T> static T Call(T Value) { return FMath::CeilToFloat(Value); } };
struct FOpCos { template<typename T> static T Call(T Value) { return FMath::Cos(Value); } };
struct FOpExponential { template<typename T> static T Call(T Value) { return FMath::Exp(Value); } };
struct FOpExponential2 { template<typename T> static T Call(T Value) { return FMath::Exp2(Value); } };
struct FOpFloor { template<typename T> static T Call(T Value) { return FMath::FloorToFloat(Value); } };
struct FOpFrac { template<typename T> static T Call(T Value) { return FMath::Frac(Value); } };
struct FOpFractional { template<typename T> static T Call(T Value) { return FMath::Fractional(Value); } };
struct FOpLogarithm { template<typename T> static T Call(T Value) { return FMath::Loge(Value); } };
struct FOpLogarithm2 { template<typename T> static T Call(T Value) { return FMath::Log2(Value); } };
struct FOpLogarithm10 { template<typename T> static T Call(T Value)
{
	static const T LogToLog10 = (T)1.0 / FMath::Loge((T)10);
	return FMath::Loge(Value) * LogToLog10;
}};
struct FOpNegate { template<typename T> static T Call(T Value) { return -Value; } };
struct FOpReciprocal { template<typename T> static T Call(T Value) { return (T)1 / GetSafeDivisor(Value); } };
struct FOpRound { template<typename T> static T Call(T Value) { return FMath::RoundToFloat(Value); } };
struct FOpSaturate { template<typename T> static T Call(T Value) { return FMath::Clamp(Value, (T)0, (T)1); } };
struct FOpSign { template<typename T> static T Call(T Value) { return FMath::Sign(Value); } };
struct FOpSin { template<typename T> static T Call(T Value) { return FMath::Sin(Value); } };
struct FOpSqrt { template<typename T> static T Call(T Value) { return FMath::Sqrt(Value); } };
struct FOpTan { template<typename T> static T Call(T Value) { return FMath::Tan(Value); } };
struct FOpTruncate { template<typename T> static T Call(T Value) { return FMath::TruncToFloat(Value); } };

struct FOpAdd { template<typename T> static T Call(T A, T B) { return A + B; } };
struct FOpSubtract { template<typename T> static T Call(T A, T B) { return A - B; } };
struct FOpMultiply { template<typename T> static T Call(T A, T B) { return A * B; } };
struct FOpDivide { template<typename T> static T Call(T A, T B) { return A / GetSafeDivisor(B); } };
struct FOpMin { template<typename T> static T Call(T A, T B) { return FMath::Min(A, B); } };
struct FOpMax { template<typename T> static T Call(T A, T B) { return FMath::Max(A, B); } };
struct FOpATan2 { template<typename T> static T Call(T A, T B) { return FMath::Atan2(A, B); } };

// Note:  we are not defining a separate opcode for Fmod and Modulo.  Operations on integer will do Modulo, while operations on float will do Fmod.
struct FOpFmod
{
	static float Call(float A, float B) { return FMath::Fmod(A, GetSafeDivisor(B)); }
	static double Call(double A, double B) { return FMath::Fmod(A, GetSafeDivisor(B)); }
	static int32 Call(int32 A, int32 B) { return A % GetSafeDivisor(B); }
};

// Ops that aren't per-component.  These are dummy structures for template specialization or overloading, and don't need an operator implementation.
// Unary.
struct FOpCast {};
struct FOpLength {};
struct FOpNormalize {};

// Binary.
struct FOpCross {};
struct FOpDot {};

// Custom.
struct FOpConstant {};
struct FOpMove {};
struct FOpMoveForFixup {};

// External Inputs.
struct FOpExternalTextureCoordinateOffset {};
struct FOpExternalTextureCoordinateScaleRotation {};
struct FOpRuntimeVirtualTextureUniform {};
struct FOpSparseVolumeTextureUniform {};
struct FOpTexelSize {};
struct FOpTextureSize {};

// Forward declarations.
static void AppendOffset(TMemoryImageArray<uint8>& Opcodes, uint16 Offset);
static void EnablePreshader2(UE::Shader::FPreshaderData& PreshaderData);

static float& AsFloat(void* Buffer, int32 Offset, int32 Index)
{
	return ((float*)Buffer)[Offset + Index];
}

static double& AsDouble(void* Buffer, int32 Offset, int32 Index)
{
	// Offset is always in 32-bit words, while Index applies to the double type, so the Index gets doubled.
	return (double&)((int32*)Buffer)[Offset + Index*2];
}

static int32& AsInt(void* Buffer, int32 Offset, int32 Index)
{
	return ((int32*)Buffer)[Offset + Index];
}

static uint16 FetchOffset(const uint8* Ptr)
{
	return (uint16)Ptr[0] | ((uint16)Ptr[1] << 8);
}

// Increments OpcodePtr by the size of T.
template<typename T>
void FetchDataValue(const uint8*& OpcodePtr, T& Value)
{
	FMemory::Memcpy(&Value, OpcodePtr, sizeof(T));
	OpcodePtr += sizeof(T);
}

// Bounds checking utility functions -- we want to bounds check once, rather than per component in vector operations, to reduce overhead in non-shipping builds.
static FORCEINLINE void BoundsCheck(TArrayView<float> Buffer, int32 Offset, int32 Dimension, bool bDouble = false)
{
	if (bDouble)
	{
		Dimension *= 2;
	}
	check(Offset + Dimension <= Buffer.Num());
}

static FORCEINLINE void BoundsCheck(TArrayView<float> Buffer, int32 ResultOffset, int32 AOffset, int32 Dimension, bool bDouble = false)
{
	if (bDouble)
	{
		Dimension *= 2;
	}
	check(FMath::Max(AOffset, ResultOffset) + Dimension <= Buffer.Num());
}

static FORCEINLINE void BoundsCheck(TArrayView<float> Buffer, int32 ResultOffset, int32 AOffset, int32 BOffset, int32 Dimension, bool bAScalar, bool bBScalar, bool bDouble = false)
{
	if (bDouble)
	{
		Dimension *= 2;
	}
	check(FMath::Max3(ResultOffset + Dimension, AOffset + (bAScalar ? 1 : Dimension), BOffset + (bBScalar ? 1 : Dimension)) <= Buffer.Num());
}

// UnaryEvaluate accepts Type as a reference, as the Normalize op changes the type of its output to float.
template<typename TOp, bool bAllowInt>
void UnaryEvaluate(TArrayView<float> Buffer, EPreshader2Type& Type, int32 Dimension, int32 ResultOffset, int32 AOffset)
{
	// By design, the macros that implement unary ops pass the Dimension byte unmodified, to allow clients to store data in it.
	// Mask the low bits to get the actual dimension.
	int32 LocalDimension = Dimension & 0x7;
	float* BufferData = Buffer.GetData();

	if (LIKELY(Type == EPreshader2Type::Float))
	{
		BoundsCheck(Buffer, ResultOffset, AOffset, LocalDimension);

		// This binary tree based on the dimension generates more compact code than a "for" loop for float / double types.
		float A = TOp::Call(AsFloat(BufferData, AOffset, 0));
		if (LocalDimension >= 3)
		{
			float B = TOp::Call(AsFloat(BufferData, AOffset, 1));
			float C = TOp::Call(AsFloat(BufferData, AOffset, 2));
			if (LocalDimension == 4)
			{
				AsFloat(BufferData, ResultOffset, 3) = TOp::Call(AsFloat(BufferData, AOffset, 3));
			}
			AsFloat(BufferData, ResultOffset, 1) = B;
			AsFloat(BufferData, ResultOffset, 2) = C;
		}
		else if (LocalDimension == 2)
		{
			AsFloat(BufferData, ResultOffset, 1) = TOp::Call(AsFloat(BufferData, AOffset, 1));
		}
		AsFloat(BufferData, ResultOffset, 0) = A;
	}
	else if (!bAllowInt || Type == EPreshader2Type::Double)
	{
		BoundsCheck(Buffer, ResultOffset, AOffset, LocalDimension, /*bDouble=*/ true);

		double A = TOp::Call(AsDouble(BufferData, AOffset, 0));
		if (LocalDimension >= 3)
		{
			double B = TOp::Call(AsDouble(BufferData, AOffset, 1));
			double C = TOp::Call(AsDouble(BufferData, AOffset, 2));
			if (LocalDimension == 4)
			{
				AsDouble(BufferData, ResultOffset, 3) = TOp::Call(AsDouble(BufferData, AOffset, 3));
			}
			AsDouble(BufferData, ResultOffset, 1) = B;
			AsDouble(BufferData, ResultOffset, 2) = C;
		}
		else if (LocalDimension == 2)
		{
			AsDouble(BufferData, ResultOffset, 1) = TOp::Call(AsDouble(BufferData, AOffset, 1));
		}
		AsDouble(BufferData, ResultOffset, 0) = A;
	}
	else if constexpr (bAllowInt)
	{
		BoundsCheck(Buffer, ResultOffset, AOffset, LocalDimension);

		for (int32 Index = 0; Index < LocalDimension; ++Index)
		{
			AsInt(BufferData, ResultOffset, Index) = TOp::Call(AsInt(BufferData, AOffset, Index));
		}
	}
}

// Specialization of UnaryEvaluate for Normalize
template <>
void UnaryEvaluate<FOpNormalize, false>(TArrayView<float> Buffer, EPreshader2Type& Type, int32 Dimension, int32 ResultOffset, int32 AOffset)
{
	// By design, the macros that implement unary ops pass the Dimension byte unmodified, to allow clients to store data in it.
	// Mask the low bits to get the actual dimension.
	int32 LocalDimension = Dimension & 0x7;
	float* BufferData = Buffer.GetData();

	if (LIKELY(Type == EPreshader2Type::Float))
	{
		BoundsCheck(Buffer, ResultOffset, AOffset, LocalDimension);

		float MagnitudeScale = 0.0f;
		for (int32 Index = 0; Index < LocalDimension; ++Index)
		{
			MagnitudeScale += FMath::Square(AsFloat(BufferData, AOffset, Index));
		}
		MagnitudeScale = FMath::InvSqrt(FMath::Max(MagnitudeScale, (float)UE_DELTA));

		for (int32 Index = 0; Index < LocalDimension; ++Index)
		{
			AsFloat(BufferData, ResultOffset, Index) = AsFloat(BufferData, AOffset, Index) * MagnitudeScale;
		}
	}
	else
	{
		// Result is float, A is double
		BoundsCheck(Buffer, ResultOffset, LocalDimension);
		BoundsCheck(Buffer, AOffset, LocalDimension, /*bDouble=*/ true);

		// Note that normalize automatically downcasts double inputs to float!
		double MagnitudeScale = 0.0f;
		for (int32 Index = 0; Index < LocalDimension; ++Index)
		{
			MagnitudeScale += FMath::Square(AsDouble(BufferData, AOffset, Index));
		}
		MagnitudeScale = FMath::InvSqrt(FMath::Max(MagnitudeScale, (double)UE_DELTA));

		for (int32 Index = 0; Index < LocalDimension; ++Index)
		{
			AsFloat(BufferData, ResultOffset, Index) = (float)(AsDouble(BufferData, AOffset, Index) * MagnitudeScale);
		}

		// Tell the caller we changed the type
		Type = EPreshader2Type::Float;
	}
}

// Specialization of UnaryEvaluate for Cast.
template <>
void UnaryEvaluate<FOpCast, true>(TArrayView<float> Buffer, EPreshader2Type& Type, int32 Dimension, int32 ResultOffset, int32 AOffset)
{
	// By design, the macros that implement unary ops pass the Dimension byte unmodified, to allow clients to store data in it.
	// Mask the low bits to get the actual dimension.
	int32 LocalDimension = Dimension & 0x7;
	float* BufferData = Buffer.GetData();

	// The cast op includes the source type in the Dimension byte.
	EPreshader2Type DestType = EPreshader2Type((Dimension >> 4) & 0x3);

	if (DestType == EPreshader2Type::Float)
	{
		if (Type == EPreshader2Type::Double)
		{
			// Result is float, A is double
			BoundsCheck(Buffer, ResultOffset, LocalDimension);
			BoundsCheck(Buffer, AOffset, LocalDimension, /*bDouble=*/ true);

			for (int32 Index = 0; Index < LocalDimension; ++Index)
			{
				AsFloat(BufferData, ResultOffset, Index) = (float)AsDouble(BufferData, AOffset, Index);
			}
		}
		else
		{
			BoundsCheck(Buffer, ResultOffset, AOffset, LocalDimension);

			// EPreshader2Type::Integer
			for (int32 Index = 0; Index < LocalDimension; ++Index)
			{
				AsFloat(BufferData, ResultOffset, Index) = (float)AsInt(BufferData, AOffset, Index);
			}
		}
	}
	else if (DestType == EPreshader2Type::LWC || DestType == EPreshader2Type::Double)
	{
		// We have to consider the possibility of in-place conversion (writing LWC or Double in place to a location that contains
		// the source data of a smaller type we are reading from), so store values to a temporary array in a first pass.
		double Temp[4];
		if (Type == EPreshader2Type::Float)
		{
			BoundsCheck(Buffer, AOffset, LocalDimension);

			for (int32 Index = 0; Index < LocalDimension; ++Index)
			{
				Temp[Index] = AsFloat(BufferData, AOffset, Index);
			}
		}
		else if (Type == EPreshader2Type::Double)
		{
			BoundsCheck(Buffer, AOffset, LocalDimension, /*bDouble=*/ true);

			for (int32 Index = 0; Index < LocalDimension; ++Index)
			{
				Temp[Index] = AsDouble(BufferData, AOffset, Index);
			}
		}
		else
		{
			// EPreshader2Type::Integer
			BoundsCheck(Buffer, AOffset, LocalDimension);

			for (int32 Index = 0; Index < LocalDimension; ++Index)
			{
				Temp[Index] = AsInt(BufferData, AOffset, Index);
			}
		}

		BoundsCheck(Buffer, ResultOffset, LocalDimension, /*bDouble=*/ true);

		if (DestType == EPreshader2Type::LWC)
		{
			for (int32 Index = 0; Index < LocalDimension; ++Index)
			{
				FLargeWorldRenderScalar LWCValue(Temp[Index]);
				AsFloat(BufferData, ResultOffset, Index*2 + 0) = LWCValue.GetTile();
				AsFloat(BufferData, ResultOffset, Index*2 + 1) = LWCValue.GetOffset();
			}
		}
		else
		{
			for (int32 Index = 0; Index < LocalDimension; ++Index)
			{
				AsDouble(BufferData, ResultOffset, Index) = Temp[Index];
			}
		}
	}
	else if (DestType == EPreshader2Type::Integer)
	{
		if (Type == EPreshader2Type::Float)
		{
			BoundsCheck(Buffer, ResultOffset, AOffset, LocalDimension);

			for (int32 Index = 0; Index < LocalDimension; ++Index)
			{
				AsInt(BufferData, ResultOffset, Index) = (int32)AsFloat(BufferData, AOffset, Index);
			}
		}
		else
		{
			// EPreshader2Type::Double
			// Result is integer, A is double
			BoundsCheck(Buffer, ResultOffset, LocalDimension);
			BoundsCheck(Buffer, AOffset, LocalDimension, /*bDouble=*/ true);

			for (int32 Index = 0; Index < LocalDimension; ++Index)
			{
				AsInt(BufferData, ResultOffset, Index) = (int32)AsDouble(BufferData, AOffset, Index);
			}
		}
	}

	// Update Type to DestType, as Type is preserved across loop iterations for Reuse opcode variants.
	Type = DestType;
}

// Specialization of UnaryEvaluate for Length.
static void UnaryEvaluate(const FOpLength&, TArrayView<float> Buffer, EPreshader2Type Type, int32& Dimension, int32 ResultOffset, int32 AOffset)
{
	// By design, the macros that implement unary ops pass the Dimension byte unmodified, to allow clients to store data in it.
	// Mask the low bits to get the actual dimension.
	int32 LocalDimension = Dimension & 0x7;
	float* BufferData = Buffer.GetData();

	// Override "Dimension" to 1, indicating output is a scalar!
	Dimension = 1;

	if (LIKELY(Type == EPreshader2Type::Float))
	{
		BoundsCheck(Buffer, AOffset, LocalDimension);
		BoundsCheck(Buffer, ResultOffset, 1);

		float MagnitudeScale = 0.0f;
		for (int32 Index = 0; Index < LocalDimension; ++Index)
		{
			MagnitudeScale += FMath::Square(AsFloat(BufferData, AOffset, Index));
		}
		AsFloat(BufferData, ResultOffset, 0) = FMath::Sqrt(MagnitudeScale);
	}
	else
	{
		BoundsCheck(Buffer, AOffset, LocalDimension, /*bDouble=*/ true);
		BoundsCheck(Buffer, ResultOffset, 1, /*bDouble=*/ true);

		double MagnitudeScale = 0.0f;
		for (int32 Index = 0; Index < LocalDimension; ++Index)
		{
			MagnitudeScale += FMath::Square(AsDouble(BufferData, AOffset, Index));
		}
		AsDouble(BufferData, ResultOffset, 0) = FMath::Sqrt(MagnitudeScale);
	}
}

// Evaluate a standard per component binary op.  AInc and BInc are array increments which allow for either input argument to be a scalar.
// A zero increment will use the same input component for each item.
template<typename TOp, bool bAllowInt>
void BinaryEvaluate(TArrayView<float> Buffer, EPreshader2Type Type, int32 Dimension, int32 ResultOffset, int32 AOffset, int32 BOffset, int32 AInc, int32 BInc)
{
	float* BufferData = Buffer.GetData();

	if (LIKELY(Type == EPreshader2Type::Float))
	{
		BoundsCheck(Buffer, ResultOffset, AOffset, BOffset, Dimension, AInc == 0, BInc == 0);

		// We do index 0 last, to handle in-place overwriting of result scalar with vector
		for (int32 Index = 1; Index < Dimension; ++Index)
		{
			AsFloat(BufferData, ResultOffset, Index) = TOp::Call(AsFloat(BufferData, AOffset, AInc*Index), AsFloat(BufferData, BOffset, BInc*Index));
		}
		AsFloat(BufferData, ResultOffset, 0) = TOp::Call(AsFloat(BufferData, AOffset, 0), AsFloat(BufferData, BOffset, 0));
	}
	else if (!bAllowInt || Type == EPreshader2Type::Double)
	{
		BoundsCheck(Buffer, ResultOffset, AOffset, BOffset, Dimension, AInc == 0, BInc == 0, /*bDouble=*/ true);

		for (int32 Index = 1; Index < Dimension; ++Index)
		{
			AsDouble(BufferData, ResultOffset, Index) = TOp::Call(AsDouble(BufferData, AOffset, AInc*Index), AsDouble(BufferData, BOffset, BInc*Index));
		}
		AsDouble(BufferData, ResultOffset, 0) = TOp::Call(AsDouble(BufferData, AOffset, 0), AsDouble(BufferData, BOffset, 0));
	}
	else if constexpr (bAllowInt)
	{
		BoundsCheck(Buffer, ResultOffset, AOffset, BOffset, Dimension, AInc == 0, BInc == 0);

		for (int32 Index = 1; Index < Dimension; ++Index)
		{
			AsInt(BufferData, ResultOffset, Index) = TOp::Call(AsInt(BufferData, AOffset, AInc*Index), AsInt(BufferData, BOffset, BInc*Index));
		}
		AsInt(BufferData, ResultOffset, 0) = TOp::Call(AsInt(BufferData, AOffset, 0), AsInt(BufferData, BOffset, 0));
	}
}

// Variant of binary evaluate that accepts an immediate constant.
template<typename TOp, bool bAllowInt>
void BinaryEvaluateConst(TArrayView<float> Buffer, EPreshader2Type Type, int32 Dimension, int32 ResultOffset, int32 AOffset, const uint8*& BData, int32 AInc, int32 BInc)
{
	float* BufferData = Buffer.GetData();

	// Copy data to an aligned buffer
	double BDataCopy[4];
	int32 CopyNum = BInc ? Dimension : 1;
	int32 CopySize = CopyNum * (Type == EPreshader2Type::Double ? 8 : 4);
	FMemory::Memcpy(BDataCopy, BData, CopySize);

	// Increment input pointer by copy size
	BData += CopySize;

	if (LIKELY(Type == EPreshader2Type::Float))
	{
		BoundsCheck(Buffer, AOffset, AInc ? Dimension : 1);
		BoundsCheck(Buffer, ResultOffset, Dimension);

		// We do index 0 last, to handle in-place overwriting of result scalar with vector
		for (int32 Index = 1; Index < Dimension; ++Index)
		{
			AsFloat(BufferData, ResultOffset, Index) = TOp::Call(AsFloat(BufferData, AOffset, AInc*Index), AsFloat(BDataCopy, 0, BInc*Index));
		}
		AsFloat(BufferData, ResultOffset, 0) = TOp::Call(AsFloat(BufferData, AOffset, 0), AsFloat(BDataCopy, 0, 0));
	}
	else if (!bAllowInt || Type == EPreshader2Type::Double)
	{
		BoundsCheck(Buffer, AOffset, AInc ? Dimension : 1, /*bDouble=*/ true);
		BoundsCheck(Buffer, ResultOffset, Dimension, /*bDouble=*/ true);

		for (int32 Index = 1; Index < Dimension; ++Index)
		{
			AsDouble(BufferData, ResultOffset, Index) = TOp::Call(AsDouble(BufferData, AOffset, AInc*Index), AsDouble(BDataCopy, 0, BInc*Index));
		}
		AsDouble(BufferData, ResultOffset, 0) = TOp::Call(AsDouble(BufferData, AOffset, 0), AsDouble(BDataCopy, 0, 0));
	}
	else if constexpr (bAllowInt)
	{
		BoundsCheck(Buffer, AOffset, AInc ? Dimension : 1);
		BoundsCheck(Buffer, ResultOffset, Dimension);

		for (int32 Index = 1; Index < Dimension; ++Index)
		{
			AsInt(BufferData, ResultOffset, Index) = TOp::Call(AsInt(BufferData, AOffset, AInc*Index), AsInt(BDataCopy, 0, BInc*Index));
		}
		AsInt(BufferData, ResultOffset, 0) = TOp::Call(AsInt(BufferData, AOffset, 0), AsInt(BDataCopy, 0, 0));
	}
}

template<typename T>
void CrossCompute(T* Result, T* A, T* B, int32 AInc, int32 BInc)
{
	T Temp[3];
	Temp[0] = A[1*AInc] * B[2*BInc] - A[2*AInc] * B[1*BInc];
	Temp[1] = A[2*AInc] * B[0*BInc] - A[0*AInc] * B[2*BInc];
	Temp[2] = A[0*AInc] * B[1*BInc] - A[1*AInc] * B[0*BInc];
	Result[0] = Temp[0];
	Result[1] = Temp[1];
	Result[2] = Temp[2];
}

// Specialization of BinaryEvaluate for Cross.
static void BinaryEvaluate(const FOpCross&, TArrayView<float> Buffer, EPreshader2Type Type, int32 Dimension, int32 ResultOffset, int32 AOffset, int32 BOffset, int32 AInc, int32 BInc)
{
	float* BufferData = Buffer.GetData();

	// Ignores dimension, assuming dimension of 3 (we check this at creation of the op, not here at runtime)
	if (LIKELY(Type == EPreshader2Type::Float))
	{
		BoundsCheck(Buffer, ResultOffset, AOffset, BOffset, 3, AInc == 0, BInc == 0);
		CrossCompute(&AsFloat(BufferData, ResultOffset, 0), &AsFloat(BufferData, AOffset, 0), &AsFloat(BufferData, BOffset, 0), AInc, BInc);
	}
	else if (Type == EPreshader2Type::Double)
	{
		BoundsCheck(Buffer, ResultOffset, AOffset, BOffset, 3, AInc == 0, BInc == 0, /*bDouble=*/ true);
		CrossCompute(&AsDouble(BufferData, ResultOffset, 0), &AsDouble(BufferData, AOffset, 0), &AsDouble(BufferData, BOffset, 0), AInc, BInc);
	}
	else
	{
		BoundsCheck(Buffer, ResultOffset, AOffset, BOffset, 3, AInc == 0, BInc == 0);
		CrossCompute(&AsInt(BufferData, ResultOffset, 0), &AsInt(BufferData, AOffset, 0), &AsInt(BufferData, BOffset, 0), AInc, BInc);
	}
}

// Specialization of BinaryEvaluate for Dot.  Accepts Dimension as a reference by design!
static void BinaryEvaluate(const FOpDot&, TArrayView<float> Buffer, EPreshader2Type Type, int32& Dimension, int32 ResultOffset, int32 AOffset, int32 BOffset, int32 AInc, int32 BInc)
{
	float* BufferData = Buffer.GetData();

	if (LIKELY(Type == EPreshader2Type::Float))
	{
		BoundsCheck(Buffer, AOffset, AInc ? Dimension : 1);
		BoundsCheck(Buffer, BOffset, BInc ? Dimension : 1);
		BoundsCheck(Buffer, ResultOffset, 1);

		float DotSum = 0.0f;
		for (int32 Index = 0; Index < Dimension; ++Index)
		{
			DotSum += AsFloat(BufferData, AOffset, AInc*Index) * AsFloat(BufferData, BOffset, BInc*Index);
		}
		AsFloat(BufferData, ResultOffset, 0) = DotSum;
	}
	else
	{
		BoundsCheck(Buffer, AOffset, AInc ? Dimension : 1, /*bDouble=*/ true);
		BoundsCheck(Buffer, BOffset, BInc ? Dimension : 1, /*bDouble=*/ true);
		BoundsCheck(Buffer, ResultOffset, 1, /*bDouble=*/ true);

		double DotSum = 0.0;
		for (int32 Index = 0; Index < Dimension; ++Index)
		{
			DotSum += AsDouble(BufferData, AOffset, AInc*Index) * AsDouble(BufferData, BOffset, BInc*Index);
		}
		AsDouble(BufferData, ResultOffset, 0) = DotSum;
	}

	// Override "Dimension" to 1, indicating output is a scalar.
	Dimension = 1;
}

// OpEvaluate functions accept references to and modify Type, Dimension, and ResultOffset, as the subsequent op may reuse these.
static void OpEvaluate(const FOpConstant&, const FMaterialRenderContext* Context, TArrayView<const FScriptName> Names, TArrayView<float> Buffer, const uint8*& OpcodePtr, EPreshader2Type& Type, int32& Dimension, int32& ResultOffset)
{
	// Opcode(1), ResultOffset(2), Dimension+Type(1), Data(N)
	ResultOffset = FetchOffset(OpcodePtr);
	Dimension = OpcodePtr[2];
	Type = (EPreshader2Type)(Dimension >> 6);
	Dimension = Dimension & 0x7;

	const uint8* DataPtr = OpcodePtr + 3;
	int32 ElementSize = Type == EPreshader2Type::Double ? 8 : 4;
	int32 CopySize = Dimension * ElementSize;
	FMemory::Memcpy(&Buffer[ResultOffset], DataPtr, CopySize);

	OpcodePtr = DataPtr + CopySize;
}

static void OpFixup(const FOpConstant&, UE::Shader::FPreshaderData& PreshaderOut, TConstArrayView<uint16> Remap, const uint8*& OpcodePtr, EPreshader2Type& Type, int32& Dimension, int32& ResultOffset)
{
	// Opcode(1), ResultOffset(2), Dimension+Type(1), Data(N)
	ResultOffset = Remap[FetchOffset(OpcodePtr)];
	Dimension = OpcodePtr[2];
	Type = (EPreshader2Type)(Dimension >> 6);
	Dimension = Dimension & 0x7;

	const uint8* DataPtr = OpcodePtr + 3;
	int32 ElementSize = Type == EPreshader2Type::Double ? 8 : 4;
	int32 CopySize = Dimension * ElementSize;

	EmitConstantOp(PreshaderOut, Type, Dimension, ResultOffset, DataPtr);

	OpcodePtr = DataPtr + CopySize;
}

static void OpEvaluate(const FOpMove&, const FMaterialRenderContext* Context, TArrayView<const FScriptName> Names, TArrayView<float> Buffer, const uint8*& OpcodePtr, EPreshader2Type& Type, int32& Dimension, int32& ResultOffset)
{
	float* BufferData = Buffer.GetData();

	uint8 CountData = OpcodePtr[0];
	ResultOffset = FetchOffset(OpcodePtr + 1);
	OpcodePtr += 3;

	// MoveNum and MoveSizes here are all one less than their actual count
	uint8 MoveNum = (CountData & 0x3);
	uint8 MoveSizes[4] =
	{
		((CountData >> 2) & 0x3),			// 2 bit first move size
		((CountData >> 4) & 0x3),			// 2 bit second move size
		((ResultOffset >> 15) & 0x1),		// 1 bit third move size
		0									// 0 bit implicit fourth move size
	};
	Type = (EPreshader2Type)(CountData >> 6);
	ResultOffset &= 0x7fff;
	Dimension = 0;

	if (UNLIKELY(Type == EPreshader2Type::Double))
	{
		// Need to copy to temporary array first to allow for in-place moves
		double Temp[4];

		// Less than or equals compares because MoveNum and MoveSizes are one less than the actual count.
		for (int32 MoveIndex = 0; MoveIndex <= MoveNum; ++MoveIndex, OpcodePtr += 2)
		{
			int32 SourceOffset = FetchOffset(OpcodePtr);
			BoundsCheck(Buffer, SourceOffset, MoveSizes[MoveIndex] + 1, /*bDouble=*/ true);

			for (int32 MoveWord = 0; MoveWord <= MoveSizes[MoveIndex]; ++MoveWord)
			{
				Temp[Dimension] = AsDouble(BufferData, SourceOffset, MoveWord);
				Dimension++;
			}
		}

		BoundsCheck(Buffer, ResultOffset, Dimension, /*bDouble=*/ true);
		FMemory::Memcpy(BufferData + ResultOffset, Temp, Dimension * sizeof(double));
	}
	else
	{
		// Need to copy to temporary array first to allow for in-place moves
		float Temp[4];

		// Less than or equals compares because MoveNum and MoveSizes are one less than the actual count.
		for (int32 MoveIndex = 0; MoveIndex <= MoveNum; ++MoveIndex, OpcodePtr += 2)
		{
			int32 SourceOffset = FetchOffset(OpcodePtr);
			BoundsCheck(Buffer, SourceOffset, MoveSizes[MoveIndex] + 1);

			for (int32 MoveWord = 0; MoveWord <= MoveSizes[MoveIndex]; ++MoveWord)
			{
				Temp[Dimension] = AsFloat(BufferData, SourceOffset, MoveWord);
				Dimension++;
			}
		}

		BoundsCheck(Buffer, ResultOffset, Dimension);
		FMemory::Memcpy(Buffer.GetData() + ResultOffset, Temp, Dimension * sizeof(float));
	}
}

static void OpFixup(const FOpMove&, UE::Shader::FPreshaderData& PreshaderOut, TConstArrayView<uint16> Remap, const uint8*& OpcodePtr, EPreshader2Type& Type, int32& Dimension, int32& ResultOffset)
{
	check(!"Attempting to fixup wrong version of Move op!  Use EmitMoveOpForFixup when using FixupAndCompress.");
	UE_ASSUME(false);
}

static void OpEvaluate(const FOpMoveForFixup&, const FMaterialRenderContext* Context, TArrayView<const FScriptName> Names, TArrayView<float> Buffer, const uint8*& OpcodePtr, EPreshader2Type& Type, int32& Dimension, int32& ResultOffset)
{
	check(!"Attempting to evaluate wrong version of Move op!  Must call FixupAndCompress after using EmitMoveOpForFixup.");
	UE_ASSUME(false);
}

static void OpFixup(const FOpMoveForFixup&, UE::Shader::FPreshaderData& PreshaderOut, TConstArrayView<uint16> Remap, const uint8*& OpcodePtr, EPreshader2Type& Type, int32& Dimension, int32& ResultOffset)
{
	Dimension = OpcodePtr[0];
	ResultOffset = Remap[FetchOffset(OpcodePtr + 1)];
	uint8 ComponentStart = OpcodePtr[3];
	Type = (EPreshader2Type)(Dimension >> 6);
	Dimension = Dimension & 0x7;
	OpcodePtr += 4;

	int32 ElementSizeInWords = Type == EPreshader2Type::Double ? 2 : 1;
	ResultOffset += ElementSizeInWords * ComponentStart;

	TArray<uint16, TFixedAllocator<4>> SourceOffsets;
	SourceOffsets.SetNumUninitialized(Dimension);
	for (int32 RegisterIndex = 0; RegisterIndex < Dimension; RegisterIndex++, OpcodePtr += 2)
	{
		SourceOffsets[RegisterIndex] = Remap[FetchOffset(OpcodePtr)];
	}
	for (int32 RegisterIndex = 0; RegisterIndex < Dimension; RegisterIndex++, OpcodePtr++)
	{
		SourceOffsets[RegisterIndex] += ElementSizeInWords * OpcodePtr[0];
	}

	EmitMoveOp(PreshaderOut, Type, ResultOffset, SourceOffsets);
}

static const uint8* FetchHashedMaterialParameterInfo(TArrayView<const FScriptName> Names, const uint8* OpcodePtr, FHashedMaterialParameterInfo& OutParameterInfo)
{
	// Parameter name
	uint16 NameIndex16;
	FetchDataValue(OpcodePtr, NameIndex16);
	OutParameterInfo.Name = Names[NameIndex16];

	// Other fields of FHashedMaterialParameterInfo
	FetchDataValue(OpcodePtr, OutParameterInfo.Index);
	FetchDataValue(OpcodePtr, OutParameterInfo.Association);

	return OpcodePtr;
}

// Should match "Fetch" function above, but just advances OpcodePtr without reading anything
static const uint8* SkipHashedMaterialParameterInfo(const uint8* OpcodePtr)
{
	const FHashedMaterialParameterInfo* Dummy = (const FHashedMaterialParameterInfo*)nullptr;
	return OpcodePtr + sizeof(uint16) + sizeof(Dummy->Index) + sizeof(Dummy->Association);
}

static const uint8* GetTextureParameter(const FMaterialRenderContext* Context, TArrayView<const FScriptName> Names, const uint8* OpcodePtr, const UTexture*& OutTexture, const USparseVolumeTexture*& OutSparseVolumeTexture)
{
	// Parameter info.
	FHashedMaterialParameterInfo ParameterInfo;
	OpcodePtr = FetchHashedMaterialParameterInfo(Names, OpcodePtr, ParameterInfo);

	// Other data.
	int32 TextureIndex;
	FetchDataValue(OpcodePtr, TextureIndex);

	OutTexture = nullptr;
	OutSparseVolumeTexture = nullptr;

	Context->GetTextureParameterValue(ParameterInfo, TextureIndex, OutTexture);
	if (!OutTexture)
	{
		Context->GetTextureParameterValue(ParameterInfo, TextureIndex, OutSparseVolumeTexture);
	}

	// Return new OpcodePtr, advanced during calls to FetchDataValue.
	return OpcodePtr;
}

static const uint8* SkipTextureParameter(const uint8* OpcodePtr)
{
	OpcodePtr = SkipHashedMaterialParameterInfo(OpcodePtr);
	return OpcodePtr + sizeof(int32);							// TextureIndex
}

static void OpEvaluate(const FOpRuntimeVirtualTextureUniform&, const FMaterialRenderContext* Context, TArrayView<const FScriptName> Names, TArrayView<float> Buffer, const uint8*& OpcodePtr, EPreshader2Type& Type, int32& Dimension, int32& ResultOffset)
{
	check(Context);

	ResultOffset = FetchOffset(OpcodePtr);
	Dimension = OpcodePtr[2];
	Type = (EPreshader2Type)(Dimension >> 6);
	Dimension = Dimension & 0x7;

	int32 WordCount = Dimension * (Type == EPreshader2Type::Float ? 1 : 2);
	BoundsCheck(Buffer, ResultOffset, WordCount);

	// Parameter info.
	FHashedMaterialParameterInfo ParameterInfo;
	OpcodePtr = FetchHashedMaterialParameterInfo(Names, OpcodePtr + 3, ParameterInfo);

	// Other data.
	int32 TextureIndex;
	int32 VectorIndex;
	FetchDataValue(OpcodePtr, TextureIndex);
	FetchDataValue(OpcodePtr, VectorIndex);

	const URuntimeVirtualTexture* Texture = nullptr;
	if (ParameterInfo.Name.IsNone() || !Context->MaterialRenderProxy || !Context->MaterialRenderProxy->GetTextureValue(ParameterInfo, &Texture, *Context))
	{
		Texture = GetIndexedTexture<URuntimeVirtualTexture>(Context->Material, TextureIndex);
	}
	if (Texture && VectorIndex != INDEX_NONE)
	{
		FVector4 Value = Texture->GetUniformParameter(VectorIndex);
		if (Type == EPreshader2Type::Float)
		{
			FVector4f ValueF = FVector4f(Value);
			FMemory::Memcpy(Buffer.GetData() + ResultOffset, &ValueF, Dimension * sizeof(float));
		}
		else
		{
			// Assume double types for runtime virtual textures are LWC.
			check(Type == EPreshader2Type::LWC);

			FDFVector4 ValueDF(Value);
			FMemory::Memcpy(Buffer.GetData() + ResultOffset,             &ValueDF.High[0], Dimension * sizeof(float));
			FMemory::Memcpy(Buffer.GetData() + ResultOffset + Dimension, &ValueDF.Low[0],  Dimension * sizeof(float));
		}
	}
	else
	{
		FMemory::Memset(Buffer.GetData() + ResultOffset, 0, WordCount * sizeof(float));
	}
}

static void OpFixup(const FOpRuntimeVirtualTextureUniform&, UE::Shader::FPreshaderData& PreshaderOut, TConstArrayView<uint16> Remap, const uint8*& OpcodePtr, EPreshader2Type& Type, int32& Dimension, int32& ResultOffset)
{
	// We are emitting code here without explicitly calling the Emit function, so mark this as Preshader2 enabled
	EnablePreshader2(PreshaderOut);

	ResultOffset = Remap[FetchOffset(OpcodePtr)];
	uint8 DimensionAndType = OpcodePtr[2];
	Type = (EPreshader2Type)(DimensionAndType >> 6);
	Dimension = DimensionAndType & 0x7;
	OpcodePtr += 3;

	// We can just append the data after the offset, which doesn't need fixup -- simpler than expanding the data to pass to the original emit function.
	const uint8* PayloadStart = OpcodePtr;

	OpcodePtr = SkipHashedMaterialParameterInfo(OpcodePtr);
	OpcodePtr += sizeof(int32) + sizeof(int32);		// TextureIndex, VectorIndex

	PreshaderOut.Data.Add((uint8)EPreshader2Opcode::RuntimeVirtualTextureUniform);
	AppendOffset(PreshaderOut.Data, ResultOffset);
	PreshaderOut.Data.Add(DimensionAndType);
	PreshaderOut.Data.Append(PayloadStart, OpcodePtr - PayloadStart);
}

void OpEvaluate(const FOpTexelSize&, const FMaterialRenderContext* Context, TArrayView<const FScriptName> Names, TArrayView<float> Buffer, const uint8*& OpcodePtr, EPreshader2Type& Type, int32& Dimension, int32& ResultOffset)
{
	check(Context);

	float* BufferData = Buffer.GetData();

	ResultOffset = FetchOffset(OpcodePtr);
	Dimension = OpcodePtr[2];
	Type = (EPreshader2Type)(Dimension >> 6);
	Dimension = Dimension & 0x7;

	BoundsCheck(Buffer, ResultOffset, Dimension);

	const UTexture* Texture;
	const USparseVolumeTexture* SparseVolumeTexture;
	OpcodePtr = GetTextureParameter(Context, Names, OpcodePtr + 3, Texture, SparseVolumeTexture);

	uint32 Sizes[3];
	if (Texture && Texture->GetResource())
	{
		Sizes[0] = Texture->GetResource()->GetSizeX();
		Sizes[1] = Texture->GetResource()->GetSizeY();
		Sizes[2] = Texture->GetResource()->GetSizeZ();
	}
	else if (SparseVolumeTexture)
	{
		Sizes[0] = SparseVolumeTexture->GetSizeX();
		Sizes[1] = SparseVolumeTexture->GetSizeY();
		Sizes[2] = SparseVolumeTexture->GetSizeZ();
	}
	else
	{
		for (int32 Index = 0; Index < Dimension; Index++)
		{
			AsFloat(BufferData, ResultOffset, Index) = 0.0f;
		}
		return;
	}

	for (int32 Index = 0; Index < Dimension; Index++)
	{
		AsFloat(BufferData, ResultOffset, Index) = 1.0f / (float)Sizes[Index];
	}
}

void OpEvaluate(const FOpTextureSize&, const FMaterialRenderContext* Context, TArrayView<const FScriptName> Names, TArrayView<float> Buffer, const uint8*& OpcodePtr, EPreshader2Type& Type, int32& Dimension, int32& ResultOffset)
{
	check(Context);

	float* BufferData = Buffer.GetData();

	ResultOffset = FetchOffset(OpcodePtr);
	Dimension = OpcodePtr[2];
	Type = (EPreshader2Type)(Dimension >> 6);
	Dimension = Dimension & 0x7;

	BoundsCheck(Buffer, ResultOffset, Dimension);

	const UTexture* Texture;
	const USparseVolumeTexture* SparseVolumeTexture;
	OpcodePtr = GetTextureParameter(Context, Names, OpcodePtr + 3, Texture, SparseVolumeTexture);

	uint32 Sizes[3];
	if (Texture && Texture->GetResource())
	{
		Sizes[0] = Texture->GetResource()->GetSizeX();
		Sizes[1] = Texture->GetResource()->GetSizeY();
		Sizes[2] = Texture->GetResource()->GetSizeZ();
	}
	else if (SparseVolumeTexture)
	{
		Sizes[0] = SparseVolumeTexture->GetSizeX();
		Sizes[1] = SparseVolumeTexture->GetSizeY();
		Sizes[2] = SparseVolumeTexture->GetSizeZ();
	}
	else
	{
		for (int32 Index = 0; Index < Dimension; Index++)
		{
			AsFloat(BufferData, ResultOffset, Index) = 0.0f;
		}
		return;
	}

	for (int32 Index = 0; Index < Dimension; Index++)
	{
		AsFloat(BufferData, ResultOffset, Index) = (float)Sizes[Index];
	}
}

static void FixupTextureOp(EPreshader2Opcode Opcode, UE::Shader::FPreshaderData& PreshaderOut, TConstArrayView<uint16> Remap, const uint8*& OpcodePtr, EPreshader2Type& Type, int32& Dimension, int32& ResultOffset)
{
	// We are emitting code here without explicitly calling the Emit function, so mark this as Preshader2 enabled
	EnablePreshader2(PreshaderOut);

	ResultOffset = Remap[FetchOffset(OpcodePtr)];
	uint8 DimensionAndType = OpcodePtr[2];
	Type = (EPreshader2Type)(DimensionAndType >> 6);
	Dimension = DimensionAndType & 0x7;
	OpcodePtr += 3;

	// We can just append the data after the offset, which doesn't need fixup -- simpler than expanding the data to pass to the original emit function.
	const uint8* PayloadStart = OpcodePtr;

	OpcodePtr = SkipTextureParameter(OpcodePtr);

	PreshaderOut.Data.Add((uint8)Opcode);
	AppendOffset(PreshaderOut.Data, ResultOffset);
	PreshaderOut.Data.Add(DimensionAndType);
	PreshaderOut.Data.Append(PayloadStart, OpcodePtr - PayloadStart);
}

static void OpFixup(const FOpTexelSize&, UE::Shader::FPreshaderData& PreshaderOut, TConstArrayView<uint16> Remap, const uint8*& OpcodePtr, EPreshader2Type& Type, int32& Dimension, int32& ResultOffset)
{
	FixupTextureOp(EPreshader2Opcode::TexelSize, PreshaderOut, Remap, OpcodePtr, Type, Dimension, ResultOffset);
}

static void OpFixup(const FOpTextureSize&, UE::Shader::FPreshaderData& PreshaderOut, TConstArrayView<uint16> Remap, const uint8*& OpcodePtr, EPreshader2Type& Type, int32& Dimension, int32& ResultOffset)
{
	FixupTextureOp(EPreshader2Opcode::TextureSize, PreshaderOut, Remap, OpcodePtr, Type, Dimension, ResultOffset);
}

// TODO
void OpEvaluate(const FOpExternalTextureCoordinateOffset&, const FMaterialRenderContext* Context, TArrayView<const FScriptName> Names, TArrayView<float> Buffer, const uint8*& OpcodePtr, EPreshader2Type& Type, int32& Dimension, int32& ResultOffset) { checkNoEntry(); }
void OpEvaluate(const FOpExternalTextureCoordinateScaleRotation&, const FMaterialRenderContext* Context, TArrayView<const FScriptName> Names, TArrayView<float> Buffer, const uint8*& OpcodePtr, EPreshader2Type& Type, int32& Dimension, int32& ResultOffset) { checkNoEntry(); }
void OpEvaluate(const FOpSparseVolumeTextureUniform&, const FMaterialRenderContext* Context, TArrayView<const FScriptName> Names, TArrayView<float> Buffer, const uint8*& OpcodePtr, EPreshader2Type& Type, int32& Dimension, int32& ResultOffset) { checkNoEntry(); }

void OpFixup(const FOpExternalTextureCoordinateOffset&, UE::Shader::FPreshaderData& PreshaderOut, TConstArrayView<uint16> Remap, const uint8*& OpcodePtr, EPreshader2Type& Type, int32& Dimension, int32& ResultOffset) { checkNoEntry(); }
void OpFixup(const FOpExternalTextureCoordinateScaleRotation&, UE::Shader::FPreshaderData& PreshaderOut, TConstArrayView<uint16> Remap, const uint8*& OpcodePtr, EPreshader2Type& Type, int32& Dimension, int32& ResultOffset) { checkNoEntry(); }
void OpFixup(const FOpSparseVolumeTextureUniform&, UE::Shader::FPreshaderData& PreshaderOut, TConstArrayView<uint16> Remap, const uint8*& OpcodePtr, EPreshader2Type& Type, int32& Dimension, int32& ResultOffset) { checkNoEntry(); }


// Unary opcode variants.
// Reuse:				Opcode(1)  													ResultOffset/Dimension/Type used from previous op, AOffset same as ResultOffset
// InPlace:				Opcode(1)  ResultOffset(2)  Dimension+Type(1)				AOffset same as ResultOffset
// Default:				Opcode(1)  ResultOffset(2)  AOffset(2)  Dimension+Type(1)
// Does not sanitize Dimension in case custom implementation stores extra data in that byte!  Client evaluate function should remove junk high bits from Dimension.
#define PRESHADER2_UNARY_OP(OP_NAME, ALLOW_INTEGER) \
	case EPreshader2Opcode::OP_NAME:																																	\
		OpcodePtr += 2;									/* Skip to AOffset */																							\
	case EPreshader2Opcode::OP_NAME ## _InPlace:		/* Otherwise AOffset pulled from same address as ResultOffset */												\
		ResultOffset = FetchOffset(ResultSrc);																															\
		AOffset = FetchOffset(OpcodePtr);																																\
		Dimension = OpcodePtr[2];						/* Dimension value which may include extra payload */															\
		Type = (EPreshader2Type)(Dimension >> 6);																														\
		OpcodePtr += 3;																																					\
	case EPreshader2Opcode::OP_NAME ## _Reuse:																															\
		UnaryEvaluate<FOp ## OP_NAME, ALLOW_INTEGER>(DataContext.ResultsBuffer, Type, Dimension, ResultOffset, AOffset);												\
		Dimension = Dimension & 7;						/* Dimension that's preserved across loop iterations, with payload removed */									\
		break;

// The Cast op is the only unary op that includes a separate destination type, which the source type is converted to, but there's no payload to store the type for
// "Reuse" variations, so it's pulled from the opcode.  We subtract the opcode from the first "Cast_Reuse" opcode to extract the type and add it to the Dimension,
// where it is stored in the other versions of the opcode.  To reduce code size bloat, we technically only need a case for the last version of the opcode, so we
// fall through in the prior cases, and the other two opcodes only produce the "case" statement.
#define PRESHADER2_UNARY_OP_CAST_REUSE(OP_NAME) \
	case EPreshader2Opcode::OP_NAME: [[fallthrough]];

#define PRESHADER2_UNARY_OP_CAST_REUSE_LAST(OP_NAME) \
	case EPreshader2Opcode::OP_NAME:																																		\
		UnaryEvaluate<FOpCast, true>(DataContext.ResultsBuffer, Type, Dimension | (((uint8)Opcode - (uint8)EPreshader2Opcode::Cast_Reuse) << 4), ResultOffset, AOffset);	\
		break;

// Opcode(1)  ResultOffset(2)  AOffset(2)  Dimension+Type(1).
// Does not sanitize Dimension in case custom implementation stores extra data in that byte!  Client evaluate function should remove junk high bits from Dimension.
#define PRESHADER2_UNARY_OP_CUSTOM(OP_NAME) \
	case EPreshader2Opcode::OP_NAME:																			\
		ResultOffset = FetchOffset(OpcodePtr);																	\
		AOffset = FetchOffset(OpcodePtr + 2);																	\
		Dimension = OpcodePtr[4];																				\
		OpcodePtr += 5;																							\
		Type = (EPreshader2Type)(Dimension >> 6);																\
		UnaryEvaluate(FOp ## OP_NAME(), DataContext.ResultsBuffer, Type, Dimension, ResultOffset, AOffset);		\
		break;

// Binary opcode variants.
// Reuse:				Opcode(1)  BOffset+BInc(2)														ResultOffset/Dimension/Type used from previous op, AOffset same as ResultOffset, AInc 1
// InPlace:				Opcode(1)  ResultOffset(2)  BOffset+BInc(2)  Dimension+Type+AInc(1)				AOffset same as ResultOffset
// Default:				Opcode(1)  ResultOffset(2)  AOffset(2)  BOffset+BInc(2)  Dimension+Type+AInc(1)
// ConstReuse:			Opcode(1)  BData(N)																ResultOffset/Dimension/Type used from previous op, AOffset same as ResultOffset, BInc 1, AInc 1
// ConstReuseScalar:	Opcode(1)  BData(N)																ResultOffset/Dimension/Type used from previous op, AOffset same as ResultOffset, BInc 0, AInc 1
// ConstInPlace:		Opcode(1)  ResultOffset(2)  Dimension+Type+AInc+BInc(1)  BData(N)				AOffset same as ResultOffset
// ConstDefault:		Opcode(1)  ResultOffset(2)  AOffset(2)  Dimension+Type+AInc+BInc(1)  BData(N)
#define PRESHADER2_BINARY_OP(OP_NAME, ALLOW_INTEGER) \
	case EPreshader2Opcode::OP_NAME ## _Reuse:																				\
		BOffset = FetchOffset(OpcodePtr);																					\
		OpcodePtr += 2;									/* Skip over BOffset */												\
		AInc = 1;																											\
		goto OP_NAME ## _Evaluate;																							\
	case EPreshader2Opcode::OP_NAME:																						\
		OpcodePtr += 2;									/* Skip to AOffset */												\
	case EPreshader2Opcode::OP_NAME ## _InPlace:		/* Otherwise AOffset pulled from same address as ResultOffset */	\
		ResultOffset = FetchOffset(ResultSrc);																				\
		AOffset = FetchOffset(OpcodePtr);																					\
		BOffset = FetchOffset(OpcodePtr + 2);																				\
		Dimension = OpcodePtr[4];																							\
		Type = (EPreshader2Type)(Dimension >> 6);																			\
		AInc = (Dimension >> 5) & 1;																						\
		Dimension = Dimension & 7;																							\
		OpcodePtr += 5;																										\
	OP_NAME ## _Evaluate:																									\
		BInc = (BOffset >> 15) & 1;																							\
		BOffset &= 0x7fff;																									\
		BinaryEvaluate<FOp ## OP_NAME, ALLOW_INTEGER>(DataContext.ResultsBuffer, Type, Dimension, ResultOffset, AOffset, BOffset, AInc, BInc);	\
		break;																												\
	case EPreshader2Opcode::OP_NAME ## _ConstReuse:																			\
		AInc = 1;																											\
		BInc = 1;																											\
		goto OP_NAME ## _ConstEvaluate;																						\
	case EPreshader2Opcode::OP_NAME ## _ConstReuseScalar:																	\
		AInc = 1;																											\
		BInc = 0;																											\
		goto OP_NAME ## _ConstEvaluate;																						\
	case EPreshader2Opcode::OP_NAME ## _Const:																				\
		OpcodePtr += 2;									/* Skip to AOffset */												\
	case EPreshader2Opcode::OP_NAME ## _ConstInPlace:	/* Otherwise AOffset pulled from same address as ResultOffset */	\
		ResultOffset = FetchOffset(ResultSrc);																				\
		AOffset = FetchOffset(OpcodePtr);																					\
		Dimension = OpcodePtr[2];																							\
		Type = (EPreshader2Type)(Dimension >> 6);																			\
		AInc = (Dimension >> 5) & 1;																						\
		BInc = (Dimension >> 4) & 1;																						\
		Dimension = Dimension & 7;																							\
		OpcodePtr += 3;																										\
	OP_NAME ## _ConstEvaluate:																								\
		BinaryEvaluateConst<FOp ## OP_NAME, ALLOW_INTEGER>(DataContext.ResultsBuffer, Type, Dimension, ResultOffset, AOffset, OpcodePtr, AInc, BInc);	\
		break;

// Opcode(1)  ResultOffset(2)  AOffset(2)  BOffset(2)  Dimension+Type(1)
#define PRESHADER2_BINARY_OP_CUSTOM(OP_NAME) \
	case EPreshader2Opcode::OP_NAME:																								\
		ResultOffset = FetchOffset(OpcodePtr);																						\
		AOffset = FetchOffset(OpcodePtr + 2);																						\
		BOffset = FetchOffset(OpcodePtr + 4);																						\
		Dimension = OpcodePtr[6];																									\
		Type = (EPreshader2Type)(Dimension >> 6);																					\
		AInc = (Dimension >> 5) & 1;																								\
		BInc = (Dimension >> 4) & 1;																								\
		Dimension = Dimension & 7;																									\
		OpcodePtr += 7;																												\
		BinaryEvaluate(FOp ## OP_NAME(), DataContext.ResultsBuffer, Type, Dimension, ResultOffset, AOffset, BOffset, AInc, BInc);	\
		break;

// Fully custom evaluation
#define PRESHADER2_OP(OP_NAME) \
	case EPreshader2Opcode::OP_NAME:																												\
		OpEvaluate(FOp ## OP_NAME(), MaterialContext, DataContext.Names, DataContext.ResultsBuffer, OpcodePtr, Type, Dimension, ResultOffset);		\
		break;


void Evaluate(UE::Shader::FPreshaderDataContext& DataContext, const FMaterialRenderContext* MaterialContext)
{
	const uint8* OpcodePtr = DataContext.Ptr;
	const uint8* OpcodeEnd = DataContext.EndPtr;

	// Variables that are preserved between loop iterations, for "Reuse" opcode variants
	EPreshader2Type Type = EPreshader2Type::Float;
	int32 Dimension = 0;
	int32 ResultOffset = 0;

	while (OpcodePtr < OpcodeEnd)
	{
		const EPreshader2Opcode Opcode = (EPreshader2Opcode)*OpcodePtr;
		OpcodePtr++;

		// When an operation is set to "reuse", the result offset from the previous loop iteration is used as both input and output.
		// Initialize these to the same, in case we hit that code path.
		int32 AOffset = ResultOffset;

		// When an operation is set to "in place", the result and first argument will both be fetched from the same address.
		// Store a copy of the original address before it gets advanced, in case we hit that code path.  These two statements
		// outside the switch statement avoid the need for extra branches for every switch case where fallthrough is used,
		// reducing code size.
		const uint8* ResultSrc = OpcodePtr;

		// Temporaries, to reduce clutter in the case statement macros
		int32 BOffset;
		int32 AInc;
		int32 BInc;

		switch (Opcode)
		{
			#include "Shader/Preshader2Opcodes.inl"
		}
	}
}

#undef PRESHADER2_UNARY_OP
#undef PRESHADER2_BINARY_OP
#undef PRESHADER2_UNARY_OP_CAST_REUSE
#undef PRESHADER2_UNARY_OP_CAST_REUSE_LAST
#undef PRESHADER2_UNARY_OP_CUSTOM
#undef PRESHADER2_BINARY_OP_CUSTOM
#undef PRESHADER2_OP


// --------------------------------------------------------------------------------------------------------------------------------------------------------
//
//	Opcode construction logic
//
// --------------------------------------------------------------------------------------------------------------------------------------------------------
static void AppendOffset(TMemoryImageArray<uint8>& Opcodes, uint16 Offset)
{
	Opcodes.Add((uint8)Offset);
	Opcodes.Add((uint8)(Offset >> 8));
}

static int32 AppendImmediateConstant(TMemoryImageArray<uint8>& Opcodes, EPreshader2Type Type, uint8 Dimension, const void* Data)
{
	int32 TypeSize = Type == EPreshader2Type::Double ? 8 : 4;
	Opcodes.Append((const uint8*)Data, TypeSize * Dimension);
	return TypeSize * Dimension;
}

static void EnablePreshader2(UE::Shader::FPreshaderData& PreshaderData)
{
	if (!PreshaderData.bPreshader2)
	{
		// Make sure no one previously attempted to add non-preshader2 data to the array
		check(PreshaderData.Data.IsEmpty());

		PreshaderData.bPreshader2 = true;
	}
}

// "DestType" only matters for the Cast op.
void EmitUnaryOp(UE::Shader::FPreshaderData& PreshaderData, EPreshader2Opcode Op, EPreshader2Type Type, uint8 Dimension, uint16 ResultOffset, uint16 AOffset, bool bReuse, EPreshader2Type DestType)
{
	EnablePreshader2(PreshaderData);

	// Make sure this is a Unary opcode, and isn't a variant.  The first opcode in each group of 3 is the default version of the opcode.
	// Also ensure integer types aren't used for opcodes that don't support integer, and LWC type is only supported for Cast.
	check(Op >= EPreshader2Opcode::UnaryFirst && Op <= EPreshader2Opcode::UnaryLast);
	check(Op >= EPreshader2Opcode::UnaryCustom || (((int32)Op - (int32)EPreshader2Opcode::UnaryFirst) % 3 == 0));
	check((Op < EPreshader2Opcode::UnaryNoInt || Type != EPreshader2Type::Integer));
	check(Type != EPreshader2Type::LWC);

	// Length of a single component "vector" is equivalent to Abs, just swap in that opcode.
	if (Op == EPreshader2Opcode::Length && Dimension == 1)
	{
		Op = EPreshader2Opcode::Abs;
	}

	if (Op == EPreshader2Opcode::Length)
	{
		PreshaderData.Data.Add((uint8)Op);
		AppendOffset(PreshaderData.Data, ResultOffset);
		AppendOffset(PreshaderData.Data, AOffset);
		PreshaderData.Data.Add(Dimension | ((uint8)Type << 6));
	}
	else if (bReuse)
	{
		if (Op == EPreshader2Opcode::Cast)
		{
			PreshaderData.Data.Add((uint8)Op + 2 + (uint8)DestType);	// Reuse previous result variant, with DestType embedded into opcode
		}
		else
		{
			PreshaderData.Data.Add((uint8)Op + 2);						// Reuse previous result variant
		}
	}
	else
	{
		if (ResultOffset != AOffset || PreshaderData.bPreFixup)
		{
			PreshaderData.Data.Add((uint8)Op);							// Default variant
			AppendOffset(PreshaderData.Data, ResultOffset);
		}
		else
		{
			PreshaderData.Data.Add((uint8)Op + 1);						// In place variant
		}
		AppendOffset(PreshaderData.Data, AOffset);

		if (Op == EPreshader2Opcode::Cast)
		{
			// Shouldn't emit a cast to the same type
			check(Type != DestType);
			PreshaderData.Data.Add((uint8)Dimension | ((uint8)Type << 6) | ((uint8)DestType << 4));
		}
		else
		{
			PreshaderData.Data.Add((uint8)Dimension | ((uint8)Type << 6));
		}
	}
}

void EmitBinaryOp(UE::Shader::FPreshaderData& PreshaderData, EPreshader2Opcode Op, EPreshader2Type Type, uint8 Dimension, uint16 ResultOffset, uint16 AOffset, uint16 BOffset, bool bAScalar, bool bBScalar, bool bReuse)
{
	EnablePreshader2(PreshaderData);

	// Make sure this is a Binary opcode, and isn't a variant.  The first opcode in each group of 7 is the default version of the opcode.
	// Also ensure integer types aren't used for opcodes that don't support integer, and LWC type is only supported for Cast.
	check(Op >= EPreshader2Opcode::BinaryFirst && Op <= EPreshader2Opcode::BinaryLast);
	check(Op >= EPreshader2Opcode::BinaryCustom || (((int32)Op - (int32)EPreshader2Opcode::BinaryFirst) % 7 == 0));
	check((Op < EPreshader2Opcode::BinaryNoInt || Type != EPreshader2Type::Integer));
	check(Type != EPreshader2Type::LWC);

	if (Op == EPreshader2Opcode::Dot || Op == EPreshader2Opcode::Cross)
	{
		// Cross only works with dimension 3.
		check(Op != EPreshader2Opcode::Cross || Dimension == 3);

		PreshaderData.Data.Add((uint8)Op);
		AppendOffset(PreshaderData.Data, ResultOffset);
		AppendOffset(PreshaderData.Data, AOffset);
		AppendOffset(PreshaderData.Data, BOffset);
		PreshaderData.Data.Add(Dimension | ((uint8)Type << 6) | (bAScalar ? 0 : (uint8)1 << 5) | (bBScalar ? 0 : (uint8)1 << 4));
	}
	else if (bReuse)
	{
		// Reuse previous result doesn't support scalar for the first argument, since the first argument's type is taken from the previous op.
		check(bAScalar == false);

		PreshaderData.Data.Add((uint8)Op + 2);										// Reuse previous result variant
		AppendOffset(PreshaderData.Data, BOffset | (bBScalar ? 0 : 1u << 15));		// Increment per component is zero for scalar
	}
	else
	{
		if (ResultOffset != AOffset || PreshaderData.bPreFixup)
		{
			PreshaderData.Data.Add((uint8)Op);										// Default variant
			AppendOffset(PreshaderData.Data, ResultOffset);
		}
		else
		{
			PreshaderData.Data.Add((uint8)Op + 1);									// In place variant
		}
		AppendOffset(PreshaderData.Data, AOffset);
		AppendOffset(PreshaderData.Data, BOffset | (bBScalar ? 0 : 1u << 15));
		PreshaderData.Data.Add(Dimension | ((uint8)Type << 6) | (bAScalar ? 0 : 1u << 5));
	}
}

int32 EmitBinaryConstOp(UE::Shader::FPreshaderData& PreshaderData, EPreshader2Opcode Op, EPreshader2Type Type, uint8 Dimension, uint16 ResultOffset, uint16 AOffset, const void* BData, bool bAScalar, bool bBScalar, bool bReuse)
{
	int32 ResultConstantSize;

	EnablePreshader2(PreshaderData);

	// Make sure this is a Binary opcode, and isn't a variant.  The first opcode in each group of 7 is the default version of the opcode.
	// Also ensure integer types aren't used for opcodes that don't support integer, and LWC type is only supported for Cast.
	check(Op >= EPreshader2Opcode::BinaryFirst && Op < EPreshader2Opcode::BinaryCustom);
	check(Op >= EPreshader2Opcode::BinaryCustom || (((int32)Op - (int32)EPreshader2Opcode::BinaryFirst) % 7 == 0));
	check((Op < EPreshader2Opcode::BinaryNoInt || Type != EPreshader2Type::Integer));
	check(Type != EPreshader2Type::LWC);

	if (bReuse)
	{
		// Reuse previous result doesn't support scalar for the first argument, since the first argument's type is taken from the previous op.
		check(bAScalar == false);

		PreshaderData.Data.Add((uint8)Op + (bBScalar ? 6 : 5));					// Const reuse previous variant, vector or scalar constant
		ResultConstantSize = AppendImmediateConstant(PreshaderData.Data, Type, bBScalar ? 1 : Dimension, BData);
	}
	else
	{
		if (ResultOffset != AOffset || PreshaderData.bPreFixup)
		{
			PreshaderData.Data.Add((uint8)Op + 3);									// Const default variant
			AppendOffset(PreshaderData.Data, ResultOffset);
		}
		else
		{
			PreshaderData.Data.Add((uint8)Op + 4);									// Const in place variant
		}
		AppendOffset(PreshaderData.Data, AOffset);

		// Emit an increment of zero for scalar, one for vector.
		PreshaderData.Data.Add((uint8)Dimension | ((uint8)Type << 6) | (bAScalar ? 0 : 1u << 5) | (bBScalar ? 0 : 1u << 4));

		ResultConstantSize = AppendImmediateConstant(PreshaderData.Data, Type, bBScalar ? 1 : Dimension, BData);
	}

	return ResultConstantSize;
}

void EmitConstantOp(UE::Shader::FPreshaderData& PreshaderData, EPreshader2Type Type, uint8 Dimension, uint16 ResultOffset, const void* Data)
{
	EnablePreshader2(PreshaderData);

	check(Dimension <= 4 && Dimension >= 1);
	check(Type != EPreshader2Type::LWC);

	PreshaderData.Data.Add((uint8)EPreshader2Opcode::Constant);
	AppendOffset(PreshaderData.Data, ResultOffset);
	PreshaderData.Data.Add((uint8)Dimension | ((uint8)Type << 6));

	AppendImmediateConstant(PreshaderData.Data, Type, Dimension, Data);
}

// SourceOffsets contains the offsets of individual components to be moved.  If any are adjacent, the moves can be combined.
void EmitMoveOp(UE::Shader::FPreshaderData& PreshaderData, EPreshader2Type Type, uint16 ResultOffset, TConstArrayView<uint16> SourceOffsets)
{
	EnablePreshader2(PreshaderData);
	check(!PreshaderData.bPreFixup);

	check(SourceOffsets.Num() <= 4 && SourceOffsets.Num() >= 1);
	check(Type != EPreshader2Type::LWC);

	int32 ElementSizeInWords = Type == EPreshader2Type::Double ? 2 : 1;

	// Validate offsets for double -- all offsets must be even!
	if (Type == EPreshader2Type::Double)
	{
		check((ResultOffset & 1) == 0);

		for (int32 Index = 0; Index < SourceOffsets.Num(); ++Index)
		{
			check((SourceOffsets[Index] & 1) == 0);
		}
	}

	// Move Sizes default to 1, so unused move sizes can have one subtracted and produce zero.
	uint8 MoveSizes[4] = { 1, 1, 1, 1 };
	uint16 MoveOffsets[4];
	uint8 MoveNum = 0;

	// Add first move
	MoveSizes[0] = 1;
	MoveOffsets[0] = SourceOffsets[0];
	MoveNum++;

	for (int32 Index = 1; Index < SourceOffsets.Num(); ++Index)
	{
		if (SourceOffsets[Index] == MoveOffsets[MoveNum - 1] + ElementSizeInWords * MoveSizes[MoveNum - 1])
		{
			// Add to existing move
			MoveSizes[MoveNum - 1]++;
		}
		else
		{
			// Start a new move
			MoveSizes[MoveNum] = 1;
			MoveOffsets[MoveNum] = SourceOffsets[Index];
			MoveNum++;
		}
	}

	PreshaderData.Data.Add((uint8)EPreshader2Opcode::Move);

	// For compression, MoveNum and move sizes have one subtracted.  This allows the first and second move size to fit in 2 bits, and the
	// third move size (which can be 1 or 2 elements) to fit in 1 bit.  No fourth move size is required, as it can at most be 1 element.
	PreshaderData.Data.Add(((uint8)Type << 6) | (MoveNum - 1) | ((MoveSizes[0] - 1) << 2) | ((MoveSizes[1] - 1) << 4));

	AppendOffset(PreshaderData.Data, ResultOffset | ((MoveSizes[2] - 1u) << 15));

	for (int32 Index = 0; Index < MoveNum; ++Index)
	{
		AppendOffset(PreshaderData.Data, MoveOffsets[Index]);
	}
}

void EmitMoveOpForFixup(UE::Shader::FPreshaderData& PreshaderData, EPreshader2Type Type, uint16 ResultRegister, uint8 ResultComponent, TConstArrayView<uint16> Registers, TConstArrayView<uint8> Components)
{
	EnablePreshader2(PreshaderData);
	check(PreshaderData.bPreFixup);

	check(Registers.Num() <= 4 && Registers.Num() >= 1 && Registers.Num() == Components.Num());
	check(Type != EPreshader2Type::LWC);

	PreshaderData.Data.Add((uint8)EPreshader2Opcode::MoveForFixup);
	PreshaderData.Data.Add(((uint8)Type << 6) | Registers.Num());

	AppendOffset(PreshaderData.Data, ResultRegister);
	PreshaderData.Data.Add(ResultComponent);

	for (int32 Index = 0; Index < Registers.Num(); ++Index)
	{
		AppendOffset(PreshaderData.Data, Registers[Index]);
	}
	PreshaderData.Data.Append(Components);
}

void EmitTextureOp(UE::Shader::FPreshaderData& PreshaderData, EPreshader2Opcode Op, EPreshader2Type Type, uint8 Dimension, uint16 ResultOffset, const FHashedMaterialParameterInfo& HashedSourceParameterInfo, int32 TextureIndex, int32 UniformIndex)
{
	EnablePreshader2(PreshaderData);

	check(Op >= EPreshader2Opcode::TextureFirst && Op <= EPreshader2Opcode::TextureLast);
	PreshaderData.Data.Add((uint8)Op);
	AppendOffset(PreshaderData.Data, ResultOffset);
	PreshaderData.Data.Add(Dimension | ((uint8)Type << 6));

	// Material parameter info
	PreshaderData.Write(HashedSourceParameterInfo);

	// Other data
	PreshaderData.Write(TextureIndex);

	if (Op == EPreshader2Opcode::RuntimeVirtualTextureUniform)
	{
		PreshaderData.Write(UniformIndex);
	}
}


// --------------------------------------------------------------------------------------------------------------------------------------------------------
//
//	FixupAndCompress implementation.
//
// --------------------------------------------------------------------------------------------------------------------------------------------------------

// FixupAndCompress emits new opcodes with remapped offsets, and attempts to emit compressed opcodes where possible.
#define PRESHADER2_UNARY_OP(OP_NAME, ALLOW_INTEGER) \
	case EPreshader2Opcode::OP_NAME:																														\
		ResultOffset = Remap[FetchOffset(OpcodePtr)];																										\
		AOffset = Remap[FetchOffset(OpcodePtr + 2)];																										\
		Dimension = OpcodePtr[4];								/* Dimension value which may include extra payload */										\
		Type = EPreshader2Type(Dimension >> 6);																												\
		DestType = EPreshader2Type((Dimension >> 4) & 0x3);		/* DestType is included in payload for Cast operation */									\
		Dimension = Dimension & 7;								/* Dimension with payload removed */														\
		OpcodePtr += 5;																																		\
		bReuse = (ResultOffset == AOffset) && (PreviousResultOffset == ResultOffset) && (PreviousType == Type) && (PreviousDimension == Dimension);			\
		EmitUnaryOp(PreshaderOut, Opcode, Type, Dimension, ResultOffset, AOffset, bReuse, DestType);														\
		break;

// FixupAndCompress should be passed input with unique registers, where reuse cannot occur, so we can leave these macros empty.
#define PRESHADER2_UNARY_OP_CAST_REUSE(OP_NAME)
#define PRESHADER2_UNARY_OP_CAST_REUSE_LAST(OP_NAME)

// Can reuse PRESHADER2_UNARY_OP
#define PRESHADER2_UNARY_OP_CUSTOM(OP_NAME) PRESHADER2_UNARY_OP(OP_NAME, false)

#define PRESHADER2_BINARY_OP(OP_NAME, ALLOW_INTEGER) \
	case EPreshader2Opcode::OP_NAME:																						\
		ResultOffset = Remap[FetchOffset(OpcodePtr)];																		\
		AOffset = Remap[FetchOffset(OpcodePtr + 2)];																		\
		BOffset = FetchOffset(OpcodePtr + 4);			/* Need to do Remap below after removing BInc from value */			\
		Dimension = OpcodePtr[6];																							\
		Type = (EPreshader2Type)(Dimension >> 6);																			\
		AInc = (Dimension >> 5) & 1;																						\
		BInc = (BOffset >> 15) & 1;																							\
		BOffset = Remap[BOffset & 0x7fff];																					\
		Dimension = Dimension & 7;																							\
		OpcodePtr += 7;																										\
		bReuse = (ResultOffset == AOffset) && (PreviousResultOffset == ResultOffset) && (PreviousType == Type) && (PreviousDimension == Dimension);		\
		EmitBinaryOp(PreshaderOut, Opcode, Type, Dimension, ResultOffset, AOffset, BOffset, AInc==0, BInc==0, bReuse);		\
		break;																												\
	case EPreshader2Opcode::OP_NAME ## _Const:																				\
		ResultOffset = Remap[FetchOffset(OpcodePtr)];																		\
		AOffset = Remap[FetchOffset(OpcodePtr + 2)];																		\
		Dimension = OpcodePtr[4];																							\
		Type = (EPreshader2Type)(Dimension >> 6);																			\
		AInc = (Dimension >> 5) & 1;																						\
		BInc = (Dimension >> 4) & 1;																						\
		Dimension = Dimension & 7;																							\
		OpcodePtr += 5;																										\
		bReuse = (ResultOffset == AOffset) && (PreviousResultOffset == ResultOffset) && (PreviousType == Type) && (PreviousDimension == Dimension) && AInc;			\
		OpcodePtr += EmitBinaryConstOp(PreshaderOut, EPreshader2Opcode::OP_NAME, Type, Dimension, ResultOffset, AOffset, OpcodePtr, AInc==0, BInc==0, bReuse);		\
		break;

#define PRESHADER2_BINARY_OP_CUSTOM(OP_NAME) \
	case EPreshader2Opcode::OP_NAME:																						\
		ResultOffset = Remap[FetchOffset(OpcodePtr)];																		\
		AOffset = Remap[FetchOffset(OpcodePtr + 2)];																		\
		BOffset = Remap[FetchOffset(OpcodePtr + 4)];																		\
		Dimension = OpcodePtr[6];																							\
		Type = (EPreshader2Type)(Dimension >> 6);																			\
		AInc = (Dimension >> 5) & 1;																						\
		BInc = (Dimension >> 4) & 1;																						\
		Dimension = Dimension & 7;																							\
		OpcodePtr += 7;																										\
		EmitBinaryOp(PreshaderOut, Opcode, Type, Dimension, ResultOffset, AOffset, BOffset, AInc==0, BInc==0);				\
		break;

#define PRESHADER2_OP(OP_NAME) \
	case EPreshader2Opcode::OP_NAME:																						\
		OpFixup(FOp ## OP_NAME(), PreshaderOut, Remap, OpcodePtr, Type, Dimension, ResultOffset);							\
		break;


void FixupAndCompress(UE::Shader::FPreshaderData& PreshaderOut, const UE::Shader::FPreshaderData& PreshaderIn, TConstArrayView<uint16> Remap)
{
	check(&PreshaderOut != &PreshaderIn && PreshaderOut.Data.IsEmpty());

	// If empty (no opcodes generated), bPreshader2 won't have been set.
	check(PreshaderIn.bPreshader2 || PreshaderIn.Data.IsEmpty());

	const uint8* OpcodePtr = PreshaderIn.Data.GetData();
	const uint8* OpcodeEnd = OpcodePtr + PreshaderIn.Data.Num();

	// Variables that are preserved between loop iterations, for "Reuse" opcode variants
	EPreshader2Type PreviousType = EPreshader2Type::Float;
	int32 PreviousDimension = 0;
	int32 PreviousResultOffset = 0;

	while (OpcodePtr < OpcodeEnd)
	{
		const EPreshader2Opcode Opcode = (EPreshader2Opcode)*OpcodePtr;
		OpcodePtr++;

		// Temporaries, to reduce clutter in the case statement macros
		EPreshader2Type Type = EPreshader2Type::Float;
		int32 Dimension = 0;
		int32 ResultOffset = 0;
		int32 AOffset;
		int32 BOffset;
		int32 AInc;
		int32 BInc;
		bool bReuse;
		EPreshader2Type DestType;

		switch (Opcode)
		{
			#include "Shader/Preshader2Opcodes.inl"

		default:
			check(!"Invalid in-place or reuse opcode variant passed to UE::Preshader2::FixupAndCompress!");
			UE_ASSUME(false);
		}

		PreviousType = Type;
		PreviousDimension = Dimension;
		PreviousResultOffset = ResultOffset;
	}

	// Copy other fields.
	PreshaderOut.bPreshader2 = PreshaderIn.bPreshader2;
	PreshaderOut.Preshader2TemporarySize = PreshaderIn.Preshader2TemporarySize;
	PreshaderOut.Names = PreshaderIn.Names;
}

#undef PRESHADER2_UNARY_OP
#undef PRESHADER2_BINARY_OP
#undef PRESHADER2_UNARY_OP_CAST_REUSE
#undef PRESHADER2_UNARY_OP_CAST_REUSE_LAST
#undef PRESHADER2_UNARY_OP_CUSTOM
#undef PRESHADER2_BINARY_OP_CUSTOM
#undef PRESHADER2_OP



// --------------------------------------------------------------------------------------------------------------------------------------------------------
//
//	Print implementation.
//
// --------------------------------------------------------------------------------------------------------------------------------------------------------

struct FPrintContext
{
	FPrintContext(TConstArrayView<FPrintParameterInfo> InParameterInfos, TConstArrayView<FScriptName> InNames, bool bInPreFixup, TArray<FString>& OutInstructions)
		: ParameterInfos(InParameterInfos), Names(InNames), bPreFixup(bInPreFixup), Instructions(OutInstructions)
	{
		uint16 MaxParameterOffset = 0;
		for (const FPrintParameterInfo& ParameterInfo : ParameterInfos)
		{
			MaxParameterOffset = FMath::Max(MaxParameterOffset, (uint16)(ParameterInfo.Offset + ParameterInfo.ComponentCount * (ParameterInfo.bDouble ? 2 : 1)));
		}

		ParameterLookup.SetNumUninitialized(MaxParameterOffset);
		FMemory::Memset(ParameterLookup.GetData(), 0xff, ParameterLookup.Num() * sizeof(uint16));

		for (int32 ParameterIndex = 0; ParameterIndex < ParameterInfos.Num(); ParameterIndex++)
		{
			const FPrintParameterInfo& ParameterInfo = ParameterInfos[ParameterIndex];
			int32 WordCount = ParameterInfo.ComponentCount * (ParameterInfo.bDouble ? 2 : 1);

			for (int32 WordIndex = 0; WordIndex < WordCount; WordIndex++)
			{
				ParameterLookup[ParameterInfo.Offset + WordIndex] = (uint16)ParameterIndex;
			}
		}
	}

	void MarkOffsetWritten(int32 Offset, int32 Dimension, bool bDouble)
	{
		if (bDouble)
		{
			Dimension *= 2;
		}
		for (int32 Index = 0; Index < Dimension; Index++)
		{
			if (Offset + Index < ParameterLookup.Num())
			{
				ParameterLookup[Offset + Index] = UINT16_MAX;
			}
		}
	}

	// OutComponentStart specifies the start component being fetched from the parameter
	const FScriptName GetParameterAtOffset(int32 Offset, int32& OutComponentStart)
	{
		if (Offset < ParameterLookup.Num() && ParameterLookup[Offset] != UINT16_MAX)
		{
			FPrintParameterInfo ParameterInfo = ParameterInfos[ParameterLookup[Offset]];
			OutComponentStart = ParameterInfo.bDouble ? (Offset - ParameterInfo.Offset) / 2 : Offset - ParameterInfo.Offset;
			return ParameterInfo.Name;
		}

		OutComponentStart = 0;
		return FScriptName();
	}

	void AppendUniformLocation(EPreshader2Type Type, int32 Dimension, int32 Offset, int32 ComponentStart, FString& InoutResult, FScriptName ParameterName = {})
	{
		if (bPreFixup)
		{
			// Pre-fixup, offsets are register indices, not offsets in the uniform buffer, so just print them as integers prefixed by underscore.
			InoutResult.Appendf(TEXT("_%d%s"), Offset, GPreshader2LinearSwizzles[ComponentStart][Dimension]);
		}
		else
		{
			// Note that the Preshader system supports locations that span float4 buffer elements, such as LWC which uses two float4 vectors.
			// Also, there's technically no alignment requirement for temporaries not accessed in the final buffer (the Preshader system sees
			// a flat array of words), and those need to print in some way (although for readability when debugging and simplicty of temporary
			// register allocation, we may choose not to use element spanning intermediates in practice).  We'll wrap to the next buffer
			// element and keep printing words that are being accessed in the current operation, for example [0].xyzw[1].xyzw or [0].w[1].xy.
			// It's not proper HLSL syntax, but it hopefully communicates the idea reasonably.
			int32 WordCount = Type == EPreshader2Type::Double || Type == EPreshader2Type::LWC ? Dimension * 2 : Dimension;
			bool bNeedComma = false;
			while (WordCount > 0)
			{
				if (bNeedComma)
				{
					InoutResult.Append(TEXT(", "));
				}
				bNeedComma = true;

				// Figure out how many words we need to print at the current offset
				int32 WordsToPrint = FMath::Min(4 - Offset % 4, WordCount);
				InoutResult.Appendf(TEXT("[%d]%s"), Offset / 4, GPreshader2LinearSwizzles[Offset % 4][WordsToPrint]);

				WordCount -= WordsToPrint;
				Offset += WordsToPrint;
			}
		}

		if (!ParameterName.IsNone())
		{
			InoutResult.Append(TEXT(" \""));
			InoutResult.Append(*ParameterName.ToString());
			InoutResult.AppendChar(TEXT('\"'));
		}
	}

	TConstArrayView<FPrintParameterInfo> ParameterInfos;

	// For each buffer location, holds what parameter was written there for quick lookup.  If something else overwrites
	// that offset, the values are cleared to UINT16_MAX.
	TArray<uint16> ParameterLookup;

	TConstArrayView<FScriptName> Names;

	bool bPreFixup;

	// Output list of instructions written to.
	TArray<FString>& Instructions;
};

static void UnaryPrint(FPrintContext& Context, EPreshader2Opcode Opcode, EPreshader2Type& Type, int32 Dimension, int32 ResultOffset, int32 AOffset)
{
	// By design, the macros that implement unary ops pass the Dimension byte unmodified, to allow clients to store data in it.
	// Mask the low bits to get the actual dimension.
	Dimension &= 0x7;

	int32 ComponentStart;
	FScriptName ParameterName = Context.GetParameterAtOffset(AOffset, ComponentStart);
	const TCHAR* TypeName = GPreshader2TypeNames[(int32)Type];
	const TCHAR* DimensionName = GPreshader2DimensionNames[Dimension];

	FString Result;
	Context.AppendUniformLocation(Type, Opcode == EPreshader2Opcode::Length ? 1 : Dimension, ResultOffset, 0, Result);
	Result.Appendf(TEXT(" = %s("), GPreshader2OpcodeNames[(int32)Opcode]);
	Context.AppendUniformLocation(Type, Dimension, AOffset, 0, Result, ParameterName);
	Result.Appendf(TEXT(")  %s%s"), TypeName, DimensionName);

	Context.Instructions.Add(Result);
	Context.MarkOffsetWritten(ResultOffset, Dimension, Type == EPreshader2Type::Double);
}

void BinaryPrint(FPrintContext& Context, EPreshader2Opcode Opcode, EPreshader2Type Type, int32 Dimension, int32 ResultOffset, int32 AOffset, int32 BOffset, int32 AInc, int32 BInc)
{
	int32 AComponent;
	int32 BComponent;
	FScriptName AParameter = Context.GetParameterAtOffset(AOffset, AComponent);
	FScriptName BParameter = Context.GetParameterAtOffset(BOffset, BComponent);
	const TCHAR* TypeName = GPreshader2TypeNames[(int32)Type];
	const TCHAR* DimensionName = GPreshader2DimensionNames[Dimension];

	FString Result;
	Context.AppendUniformLocation(Type, Opcode == EPreshader2Opcode::Dot ? 1 : Dimension, ResultOffset, 0, Result);
	Result.Appendf(TEXT(" = %s("), GPreshader2OpcodeNames[(int32)Opcode]);
	Context.AppendUniformLocation(Type, AInc ? Dimension : 1, AOffset, 0, Result, AParameter);
	Result.Append(TEXT(", "));
	Context.AppendUniformLocation(Type, BInc ? Dimension : 1, BOffset, 0, Result, BParameter);
	Result.Appendf(TEXT(")  %s%s"), TypeName, DimensionName);

	Context.Instructions.Add(Result);
	Context.MarkOffsetWritten(ResultOffset, Dimension, Type == EPreshader2Type::Double);
}

void BinaryPrintConst(FPrintContext& Context, EPreshader2Opcode Opcode, EPreshader2Type Type, int32 Dimension, int32 ResultOffset, int32 AOffset, const uint8*& BData, int32 AInc, int32 BInc)
{
	// Copy data to an aligned buffer
	double BDataCopy[4];
	int32 CopyNum = BInc ? Dimension : 1;
	int32 CopySize = CopyNum * (Type == EPreshader2Type::Double ? 8 : 4);
	FMemory::Memcpy(BDataCopy, BData, CopySize);

	// Increment input pointer by copy size
	BData += CopySize;

	int32 AComponent;
	FScriptName AParameter = Context.GetParameterAtOffset(AOffset, AComponent);
	const TCHAR* TypeName = GPreshader2TypeNames[(int32)Type];
	const TCHAR* DimensionName = GPreshader2DimensionNames[Dimension];
	const TCHAR* ADimension = GPreshader2DimensionNames[AInc ? Dimension : 1];

	FString Result;
	Context.AppendUniformLocation(Type, Dimension, ResultOffset, 0, Result);
	Result.Appendf(TEXT(" = %s("), GPreshader2OpcodeNames[(int32)Opcode]);
	Context.AppendUniformLocation(Type, AInc ? Dimension : 1, AOffset, 0, Result, AParameter);
	Result.Append(TEXT(", { "));

	int32 BNum = BInc ? Dimension : 1;
	for (int32 Index = 0; Index < BNum; Index++)
	{
		if (Index)
		{
			Result.Append(TEXT(", "));
		}
		switch (Type)
		{
		case EPreshader2Type::Float:    Result.Appendf(TEXT("%ff"), AsFloat(BDataCopy, 0, Index));  break;
		case EPreshader2Type::Integer:  Result.Appendf(TEXT("%d"), AsInt(BDataCopy, 0, Index));  break;
		case EPreshader2Type::Double:
		case EPreshader2Type::LWC:		Result.Appendf(TEXT("%lf"), AsDouble(BDataCopy, 0, Index));  break;
		}
	}
	Result.Appendf(TEXT(" })  %s%s"), TypeName, DimensionName);

	Context.Instructions.Add(Result);
	Context.MarkOffsetWritten(ResultOffset, Dimension, Type == EPreshader2Type::Double);
}

void OpPrint(const FOpMove&, FPrintContext& Context, EPreshader2Opcode Opcode, const uint8*& OpcodePtr, EPreshader2Type& Type, int32& Dimension, int32& ResultOffset)
{
	uint8 CountData = OpcodePtr[0];
	ResultOffset = FetchOffset(OpcodePtr + 1);
	OpcodePtr += 3;

	// MoveNum and MoveSizes here are all one less than their actual count
	uint8 MoveNum = (CountData & 0x3);
	uint8 MoveSizes[4] =
	{
		((CountData >> 2) & 0x3),			// 2 bit first move size
		((CountData >> 4) & 0x3),			// 2 bit second move size
		((ResultOffset >> 15) & 0x1),		// 1 bit third move size
		0									// 0 bit implicit fourth move size
	};
	Type = (EPreshader2Type)(CountData >> 6);
	ResultOffset &= 0x7fff;

	// Compute dimension first, rather than in loop below, so we can print it.
	Dimension = 0;
	for (int32 MoveIndex = 0; MoveIndex <= MoveNum; ++MoveIndex)
	{
		Dimension += MoveSizes[MoveIndex] + 1;
	}

	int32 ElementSizeInWords = Type == EPreshader2Type::Double ? 2 : 1;

	const TCHAR* TypeName = GPreshader2TypeNames[(int32)Type];
	const TCHAR* DimensionName = GPreshader2DimensionNames[MoveNum + 1];

	FString Result;
	Context.AppendUniformLocation(Type, Dimension, ResultOffset, 0, Result);
	Result.Append(TEXT(" = Move("));

	bool bFirstElement = true;
	for (int32 MoveIndex = 0; MoveIndex <= MoveNum; ++MoveIndex, OpcodePtr += 2)
	{
		int32 SourceOffset = FetchOffset(OpcodePtr);

		for (int32 MoveWord = 0; MoveWord <= MoveSizes[MoveIndex]; ++MoveWord)
		{
			if (!bFirstElement)
			{
				Result.Append(TEXT(", "));
			}
			bFirstElement = false;

			Context.AppendUniformLocation(Type, ElementSizeInWords, SourceOffset + MoveWord * ElementSizeInWords, 0, Result);
		}
	}
	Result.Appendf(TEXT(")  %s%s"), TypeName, DimensionName);

	Context.Instructions.Add(Result);
	Context.MarkOffsetWritten(ResultOffset, Dimension, Type == EPreshader2Type::Double);
}

void OpPrint(const FOpMoveForFixup&, FPrintContext& Context, EPreshader2Opcode Opcode, const uint8*& OpcodePtr, EPreshader2Type& Type, int32& Dimension, int32& ResultOffset)
{
	Dimension = OpcodePtr[0];
	ResultOffset = FetchOffset(OpcodePtr + 1);
	uint8 ComponentStart = OpcodePtr[3];
	Type = (EPreshader2Type)(Dimension >> 6);
	Dimension = Dimension & 0x7;
	OpcodePtr += 4;

	const TCHAR* TypeName = GPreshader2TypeNames[(int32)Type];
	const TCHAR* DimensionName = GPreshader2DimensionNames[Dimension];

	TArray<uint16, TFixedAllocator<4>> SourceOffsets;
	SourceOffsets.SetNumUninitialized(Dimension);
	for (int32 RegisterIndex = 0; RegisterIndex < Dimension; RegisterIndex++, OpcodePtr += 2)
	{
		SourceOffsets[RegisterIndex] = FetchOffset(OpcodePtr);
	}
	TConstArrayView<uint8> SourceRegisters(OpcodePtr, Dimension);
	OpcodePtr += Dimension;

	FString Result;
	Context.AppendUniformLocation(Type, Dimension, ResultOffset, ComponentStart, Result);
	Result.Append(TEXT(" = MoveForFixup("));

	for (int32 Index = 0; Index < Dimension; ++Index)
	{
		if (Index)
		{
			Result.Append(TEXT(", "));
		}
		Result.Appendf(TEXT("_%d%s"), (int32)SourceOffsets[Index], GPreshader2LinearSwizzles[SourceRegisters[Index]][1]);
	}
	Result.Appendf(TEXT(")  %s%s"), TypeName, DimensionName);

	Context.Instructions.Add(Result);
	Context.MarkOffsetWritten(ResultOffset, Dimension, Type == EPreshader2Type::Double);
}

void OpPrint(const FOpConstant&, FPrintContext& Context, EPreshader2Opcode Opcode, const uint8*& OpcodePtr, EPreshader2Type& Type, int32& Dimension, int32& ResultOffset)
{
	// Opcode(1), ResultOffset(2), Dimension+Type(1), Data(N)
	ResultOffset = FetchOffset(OpcodePtr);
	Dimension = OpcodePtr[2];
	Type = (EPreshader2Type)(Dimension >> 6);
	Dimension = Dimension & 0x7;

	// Copy data to an aligned buffer
	double DataCopy[4];
	const uint8* DataPtr = OpcodePtr + 3;
	int32 ElementSize = Type == EPreshader2Type::Double ? 8 : 4;
	int32 CopySize = Dimension * ElementSize;
	FMemory::Memcpy(DataCopy, DataPtr, CopySize);

	OpcodePtr = DataPtr + CopySize;

	const TCHAR* TypeName = GPreshader2TypeNames[(int32)Type];
	const TCHAR* DimensionName = GPreshader2DimensionNames[Dimension];

	FString Result;
	Context.AppendUniformLocation(Type, Dimension, ResultOffset, 0, Result);
	Result.Append(TEXT(" = Constant({ "));

	for (int32 Index = 0; Index < Dimension; Index++)
	{
		if (Index)
		{
			Result.Append(TEXT(", "));
		}
		switch (Type)
		{
		case EPreshader2Type::Float:    Result.Appendf(TEXT("%ff"), AsFloat(DataCopy, 0, Index));  break;
		case EPreshader2Type::Integer:  Result.Appendf(TEXT("%d"), AsInt(DataCopy, 0, Index));  break;
		case EPreshader2Type::Double:
		case EPreshader2Type::LWC:		Result.Appendf(TEXT("%lf"), AsDouble(DataCopy, 0, Index));  break;
		}
	}
	Result.Appendf(TEXT(" })  %s%s"), TypeName, DimensionName);

	Context.Instructions.Add(Result);
	Context.MarkOffsetWritten(ResultOffset, Dimension, Type == EPreshader2Type::Double);
}

void OpPrint(const FOpRuntimeVirtualTextureUniform&, FPrintContext& Context, EPreshader2Opcode Opcode, const uint8*& OpcodePtr, EPreshader2Type& Type, int32& Dimension, int32& ResultOffset)
{
	ResultOffset = FetchOffset(OpcodePtr);
	Dimension = OpcodePtr[2];
	Type = (EPreshader2Type)(Dimension >> 6);
	Dimension = Dimension & 0x7;

	// Parameter info.
	FHashedMaterialParameterInfo ParameterInfo;
	OpcodePtr = FetchHashedMaterialParameterInfo(Context.Names, OpcodePtr + 3, ParameterInfo);

	// Other data.
	int32 TextureIndex;
	int32 VectorIndex;
	FetchDataValue(OpcodePtr, TextureIndex);
	FetchDataValue(OpcodePtr, VectorIndex);

	FString Result;
	Context.AppendUniformLocation(Type, Dimension, ResultOffset, 0, Result);
	Result.Appendf(TEXT(" = RuntimeVirtualTexture(\"%s\", %d, %d)  %s%d"), *ParameterInfo.GetName().ToString(), TextureIndex, VectorIndex, GPreshader2TypeNames[(int32)Type], (int32)Dimension);

	Context.Instructions.Add(Result);
	Context.MarkOffsetWritten(ResultOffset, 8, false);
}

void PrintTextureOp(FPrintContext& Context, EPreshader2Opcode Opcode, const uint8*& OpcodePtr, EPreshader2Type& Type, int32& Dimension, int32& ResultOffset)
{
	ResultOffset = FetchOffset(OpcodePtr);
	Dimension = OpcodePtr[2];
	Type = (EPreshader2Type)(Dimension >> 6);
	Dimension = Dimension & 0x7;

	// Parameter info.
	FHashedMaterialParameterInfo ParameterInfo;
	OpcodePtr = FetchHashedMaterialParameterInfo(Context.Names, OpcodePtr + 3, ParameterInfo);

	// Other data.
	int32 TextureIndex;
	FetchDataValue(OpcodePtr, TextureIndex);

	// Set output type to float
	Type = EPreshader2Type::Float;

	FString Result;
	Context.AppendUniformLocation(Type, Dimension, ResultOffset, 0, Result);
	Result.Appendf(TEXT(" = %s(\"%s\", %d)  %s%d"), GPreshader2OpcodeNames[(int32)Opcode], *ParameterInfo.GetName().ToString(), TextureIndex, GPreshader2TypeNames[(int32)Type], (int32)Dimension);

	Context.Instructions.Add(Result);
	Context.MarkOffsetWritten(ResultOffset, Dimension, false);
}

void OpPrint(const FOpTexelSize&, FPrintContext& Context, EPreshader2Opcode Opcode, const uint8*& OpcodePtr, EPreshader2Type& Type, int32& Dimension, int32& ResultOffset)
{
	PrintTextureOp(Context, Opcode, OpcodePtr, Type, Dimension, ResultOffset);
}

void OpPrint(const FOpTextureSize&, FPrintContext& Context, EPreshader2Opcode Opcode, const uint8*& OpcodePtr, EPreshader2Type& Type, int32& Dimension, int32& ResultOffset)
{
	PrintTextureOp(Context, Opcode, OpcodePtr, Type, Dimension, ResultOffset);
}

void OpPrint(const FOpExternalTextureCoordinateScaleRotation&, FPrintContext& Context, EPreshader2Opcode Opcode, const uint8*& OpcodePtr, EPreshader2Type& Type, int32& Dimension, int32& ResultOffset) { checkNoEntry(); }
void OpPrint(const FOpExternalTextureCoordinateOffset&, FPrintContext& Context, EPreshader2Opcode Opcode, const uint8*& OpcodePtr, EPreshader2Type& Type, int32& Dimension, int32& ResultOffset) { checkNoEntry(); }
void OpPrint(const FOpSparseVolumeTextureUniform&, FPrintContext& Context, EPreshader2Opcode Opcode, const uint8*& OpcodePtr, EPreshader2Type& Type, int32& Dimension, int32& ResultOffset) { checkNoEntry(); }


// Cut-and-paste from Evaluate, with print functions substituted -- see comments on the original macros for details.
#define PRESHADER2_UNARY_OP(OP_NAME, ALLOW_INTEGER) \
	case EPreshader2Opcode::OP_NAME:																																	\
		OpcodePtr += 2;									/* Skip to AOffset */																							\
	case EPreshader2Opcode::OP_NAME ## _InPlace:		/* Otherwise AOffset pulled from same address as ResultOffset */												\
		ResultOffset = FetchOffset(ResultSrc);																															\
		AOffset = FetchOffset(OpcodePtr);																																\
		Dimension = OpcodePtr[2];						/* Dimension value which may include extra payload */															\
		Type = (EPreshader2Type)(Dimension >> 6);																														\
		OpcodePtr += 3;																																					\
	case EPreshader2Opcode::OP_NAME ## _Reuse:																															\
		UnaryPrint(Context, Opcode, Type, Dimension, ResultOffset, AOffset);																							\
		Dimension = Dimension & 7;						/* Dimension that's preserved across loop iterations, with payload removed */									\
		break;

#define PRESHADER2_UNARY_OP_CAST_REUSE(OP_NAME) \
	case EPreshader2Opcode::OP_NAME: [[fallthrough]];

#define PRESHADER2_UNARY_OP_CAST_REUSE_LAST(OP_NAME) \
	case EPreshader2Opcode::OP_NAME:																												\
		UnaryPrint(Context, Opcode, Type, Dimension | (((uint8)Opcode - (uint8)EPreshader2Opcode::Cast_Reuse) << 4), ResultOffset, AOffset);		\
		break;

#define PRESHADER2_UNARY_OP_CUSTOM(OP_NAME) \
	case EPreshader2Opcode::OP_NAME:																			\
		ResultOffset = FetchOffset(OpcodePtr);																	\
		AOffset = FetchOffset(OpcodePtr + 2);																	\
		Dimension = OpcodePtr[4];																				\
		OpcodePtr += 5;																							\
		Type = (EPreshader2Type)(Dimension >> 6);																\
		UnaryPrint(Context, Opcode, Type, Dimension, ResultOffset, AOffset);									\
		break;

#define PRESHADER2_BINARY_OP(OP_NAME, ALLOW_INTEGER) \
	case EPreshader2Opcode::OP_NAME ## _Reuse:																				\
		BOffset = FetchOffset(OpcodePtr);																					\
		OpcodePtr += 2;									/* Skip over BOffset */												\
		AInc = 1;																											\
		goto OP_NAME ## _Evaluate;																							\
	case EPreshader2Opcode::OP_NAME:																						\
		OpcodePtr += 2;									/* Skip to AOffset */												\
	case EPreshader2Opcode::OP_NAME ## _InPlace:		/* Otherwise AOffset pulled from same address as ResultOffset */	\
		ResultOffset = FetchOffset(ResultSrc);																				\
		AOffset = FetchOffset(OpcodePtr);																					\
		BOffset = FetchOffset(OpcodePtr + 2);																				\
		Dimension = OpcodePtr[4];																							\
		Type = (EPreshader2Type)(Dimension >> 6);																			\
		AInc = (Dimension >> 5) & 1;																						\
		Dimension = Dimension & 7;																							\
		OpcodePtr += 5;																										\
	OP_NAME ## _Evaluate:																									\
		BInc = (BOffset >> 15) & 1;																							\
		BOffset &= 0x7fff;																									\
		BinaryPrint(Context, Opcode, Type, Dimension, ResultOffset, AOffset, BOffset, AInc, BInc);							\
		break;																												\
	case EPreshader2Opcode::OP_NAME ## _ConstReuse:																			\
		AInc = 1;																											\
		BInc = 1;																											\
		goto OP_NAME ## _ConstEvaluate;																						\
	case EPreshader2Opcode::OP_NAME ## _ConstReuseScalar:																	\
		AInc = 1;																											\
		BInc = 0;																											\
		goto OP_NAME ## _ConstEvaluate;																						\
	case EPreshader2Opcode::OP_NAME ## _Const:																				\
		OpcodePtr += 2;									/* Skip to AOffset */												\
	case EPreshader2Opcode::OP_NAME ## _ConstInPlace:	/* Otherwise AOffset pulled from same address as ResultOffset */	\
		ResultOffset = FetchOffset(ResultSrc);																				\
		AOffset = FetchOffset(OpcodePtr);																					\
		Dimension = OpcodePtr[2];																							\
		Type = (EPreshader2Type)(Dimension >> 6);																			\
		AInc = (Dimension >> 5) & 1;																						\
		BInc = (Dimension >> 4) & 1;																						\
		Dimension = Dimension & 7;																							\
		OpcodePtr += 3;																										\
	OP_NAME ## _ConstEvaluate:																								\
		BinaryPrintConst(Context, Opcode, Type, Dimension, ResultOffset, AOffset, OpcodePtr, AInc, BInc);					\
		break;

#define PRESHADER2_BINARY_OP_CUSTOM(OP_NAME) \
	case EPreshader2Opcode::OP_NAME:																						\
		ResultOffset = FetchOffset(OpcodePtr);																				\
		AOffset = FetchOffset(OpcodePtr + 2);																				\
		BOffset = FetchOffset(OpcodePtr + 4);																				\
		Dimension = OpcodePtr[6];																							\
		Type = (EPreshader2Type)(Dimension >> 6);																			\
		AInc = (Dimension >> 5) & 1;																						\
		BInc = (Dimension >> 4) & 1;																						\
		Dimension = Dimension & 7;																							\
		OpcodePtr += 7;																										\
		BinaryPrint(Context, Opcode, Type, Dimension, ResultOffset, AOffset, BOffset, AInc, BInc);							\
		break;

#define PRESHADER2_OP(OP_NAME) \
	case EPreshader2Opcode::OP_NAME:																						\
		OpPrint(FOp ## OP_NAME(), Context, Opcode, OpcodePtr, Type, Dimension, ResultOffset);								\
		break;


void Print(const UE::Shader::FPreshaderData& Preshader, TConstArrayView<FPrintParameterInfo> ParameterMap, TArray<FString>& OutInstructions)
{
	FPrintContext Context(ParameterMap, Preshader.Names, Preshader.bPreFixup, OutInstructions);

	// Remaining code in this function verbatim copied from Evaluate (although macro definitions are different).
	const uint8* OpcodePtr = Preshader.Data.GetData();
	const uint8* OpcodeEnd = OpcodePtr + Preshader.Data.Num();

	// Variables that are preserved between loop iterations, for "Reuse" opcode variants
	EPreshader2Type Type = EPreshader2Type::Float;
	int32 Dimension = 0;
	int32 ResultOffset = 0;

	while (OpcodePtr < OpcodeEnd)
	{
		const EPreshader2Opcode Opcode = (EPreshader2Opcode)*OpcodePtr;
		OpcodePtr++;

		// When an operation is set to "reuse", the result offset from the previous loop iteration is used as both input and output.
		// Initialize these to the same, in case we hit that code path.
		int32 AOffset = ResultOffset;

		// When an operation is set to "in place", the result and first argument will both be fetched from the same address.
		// Store a copy of the original address before it gets advanced, in case we hit that code path.  These two statements
		// outside the switch statement avoid the need for extra branches for every switch case where fallthrough is used,
		// reducing code size.
		const uint8* ResultSrc = OpcodePtr;

		// Temporaries, to reduce clutter in the case statement macros
		int32 BOffset;
		int32 AInc;
		int32 BInc;

		switch (Opcode)
		{
			#include "Shader/Preshader2Opcodes.inl"
		}
	}
}

#undef PRESHADER2_UNARY_OP
#undef PRESHADER2_BINARY_OP
#undef PRESHADER2_UNARY_OP_CAST_REUSE
#undef PRESHADER2_UNARY_OP_CAST_REUSE_LAST
#undef PRESHADER2_UNARY_OP_CUSTOM
#undef PRESHADER2_BINARY_OP_CUSTOM
#undef PRESHADER2_OP



// --------------------------------------------------------------------------------------------------------------------------------------------------------
//
//	Unit tests
//
// --------------------------------------------------------------------------------------------------------------------------------------------------------
struct FUnitTestValue
{
	FVector4f Float;
	FVector4 Double;
	int32 Integer[4];
	float LWC[8];			// Tile / Offset pairs

	void* GetData(EPreshader2Type Type)
	{
		switch (Type)
		{
		case EPreshader2Type::Float:  return &Float;
		case EPreshader2Type::Double:  return &Double;
		case EPreshader2Type::Integer:  return &Integer[0];
		default:  return &LWC[0];
		}
	}

	const void* GetData(EPreshader2Type Type) const
	{
		return const_cast<FUnitTestValue*>(this)->GetData(Type);
	}

	static int32 GetElementSize(EPreshader2Type Type)
	{
		return ((Type == EPreshader2Type::Double) || (Type == EPreshader2Type::LWC)) ? 8 : 4;
	}

	// Initialize LWC data from the double version of the data
	void InitLWCData()
	{
		for (int32 Index = 0; Index < 4; ++Index)
		{
			FLargeWorldRenderScalar LWCValue(Double[Index]);

			LWC[Index*2 + 0] = LWCValue.GetTile();
			LWC[Index*2 + 1] = LWCValue.GetOffset();
		}
	}
};

struct FUnitTestContext
{
	const uint8 UnusedByte = 0xeb;

	EPreshader2Opcode Op;

	TStaticArray<float, 32> ResultsBuffer;
	UE::Shader::FPreshaderData PreshaderData;
	uint32 UnusedMemoryPattern[8];

	FUnitTestContext(EPreshader2Opcode InOp)
		: Op(InOp)
	{
		FMemory::Memset(UnusedMemoryPattern, UnusedByte);
		FMemory::Memset(ResultsBuffer, UnusedByte);
	}

	void SetInput(uint16 Offset, EPreshader2Type Type, int32 InputDimension, const FUnitTestValue& Input)
	{
		FMemory::Memcpy(&ResultsBuffer[Offset], Input.GetData(Type), InputDimension * FUnitTestValue::GetElementSize(Type));
	}

	void EvaluateTest(FUnitTestValue& ExpectedOutput, EPreshader2Type Type, int32 InputDimension, int32 OutputDimension = INDEX_NONE, bool bCheckUnused = true)
	{
		UE::Shader::FPreshaderDataContext DataContext(PreshaderData, ResultsBuffer);

		// If output dimension is not set, it is assumed to equal the input dimension
		OutputDimension = (OutputDimension == INDEX_NONE) ? InputDimension : OutputDimension;

		uint32 DataSize = OutputDimension * FUnitTestValue::GetElementSize(Type);

		Evaluate(DataContext);

		checkf(FMemory::Memcmp(ResultsBuffer.GetData(), ExpectedOutput.GetData(Type), DataSize) == 0, TEXT("Unit Test Fail:  %s  %d  %d"), GPreshader2OpcodeNames[(int32)Op], (int32)Type, InputDimension);

		// In-place ops that write to smaller types may not be able to unit test for unused bytes after the expected output.  In that case, we'll skip the test.
		if (bCheckUnused)
		{
			checkf(FMemory::Memcmp((uint8*)ResultsBuffer.GetData() + DataSize, UnusedMemoryPattern, 32 - DataSize) == 0, TEXT("Unit Test Failed:  %s  %d  %d"), GPreshader2OpcodeNames[(int32)Op], (int32)Type, InputDimension);
		}

		// After a successful test, reset the buffer to unused and the opcode array.
		FMemory::Memset(ResultsBuffer, UnusedByte);
		PreshaderData.Data.Empty();
	}
};

static void UnitTestUnaryOp(EPreshader2Opcode Op, FUnitTestValue& Input, FUnitTestValue& ExpectedOutput)
{
	FUnitTestContext Context(Op);
	int32 TypesToTest = Op < EPreshader2Opcode::UnaryNoInt ? 3 : 2;

	for (int32 Dimension = 1; Dimension <= 4; ++Dimension)
	{
		for (int32 TypeIndex = 0; TypeIndex < TypesToTest; ++TypeIndex)
		{
			EPreshader2Type Type = (EPreshader2Type)TypeIndex;
			uint32 DataSize = Dimension * FUnitTestValue::GetElementSize(Type);

			// Test of separate source and destination offset (data starts at offset 8, result written to offset 0)
			EmitUnaryOp(Context.PreshaderData, Op, Type, Dimension, 0, 8, false);

			Context.SetInput(8, Type, Dimension, Input);
			Context.EvaluateTest(ExpectedOutput, Type, Dimension);

			// Test of in-place result offset (data starts at offset 0, result written to offset 0)
			EmitUnaryOp(Context.PreshaderData, Op, Type, Dimension, 0, 0, false);

			Context.SetInput(0, Type, Dimension, Input);
			Context.EvaluateTest(ExpectedOutput, Type, Dimension);

			// Test of reused result offset (previous op writes a constant of the given type to offset 0, then that value is used as the input AND result of the next op)
			EmitConstantOp(Context.PreshaderData, Type, Dimension, 0, Input.GetData(Type));
			EmitUnaryOp(Context.PreshaderData, Op, Type, Dimension, 0, 0, true);

			Context.EvaluateTest(ExpectedOutput, Type, Dimension);
		}
	}
}

static void UnitTestBinaryOp(EPreshader2Opcode Op, FUnitTestValue& InputA, FUnitTestValue& InputB, bool bScalarA, bool bScalarB, FUnitTestValue& ExpectedOutput)
{
	FUnitTestContext Context(Op);
	int32 TypesToTest = Op < EPreshader2Opcode::BinaryNoInt ? 3 : 2;

	for (int32 Dimension = 1; Dimension <= 4; ++Dimension)
	{
		for (int32 TypeIndex = 0; TypeIndex < TypesToTest; ++TypeIndex)
		{
			EPreshader2Type Type = (EPreshader2Type)TypeIndex;
			uint32 DataSize = Dimension * FUnitTestValue::GetElementSize(Type);

			// Test of separate source and destination offset (data starts at offset 8 and 16, result written to offset 0)
			EmitBinaryOp(Context.PreshaderData, Op, Type, Dimension, 0, 8, 16, bScalarA, bScalarB);

			Context.SetInput(8, Type, bScalarA ? 1 : Dimension, InputA);
			Context.SetInput(16, Type, bScalarB ? 1 : Dimension, InputB);
			Context.EvaluateTest(ExpectedOutput, Type, Dimension);

			// Test of in-place result offset (data starts at offset 0 and 16, result written to offset 0)
			EmitBinaryOp(Context.PreshaderData, Op, Type, Dimension, 0, 0, 16, bScalarA, bScalarB);

			Context.SetInput(0, Type, bScalarA ? 1 : Dimension, InputA);
			Context.SetInput(16, Type, bScalarB ? 1 : Dimension, InputB);
			Context.EvaluateTest(ExpectedOutput, Type, Dimension);

			// Reuse result doesn't work if first argument is scalar
			if (bScalarA == false)
			{
				// Test of reused result offset (previous op writes a constant of the given type to offset 0, then that value is used as the input AND result of the next op)
				EmitConstantOp(Context.PreshaderData, Type, bScalarA ? 1 : Dimension, 0, InputA.GetData(Type));
				EmitBinaryOp(Context.PreshaderData, Op, Type, Dimension, 0, 0, 16, bScalarA, bScalarB, true);

				Context.SetInput(16, Type, bScalarB ? 1 : Dimension, InputB);
				Context.EvaluateTest(ExpectedOutput, Type, Dimension);
			}

			//-------------------------------------------------------------------------------------------------------------
			// Same tests repeated with immediate constant for second op
			//-------------------------------------------------------------------------------------------------------------
			EmitBinaryConstOp(Context.PreshaderData, Op, Type, Dimension, 0, 8, InputB.GetData(Type), bScalarA, bScalarB);

			Context.SetInput(8, Type, bScalarA ? 1 : Dimension, InputA);
			Context.EvaluateTest(ExpectedOutput, Type, Dimension);

			// Test of in-place result offset (data starts at offset 0, result written to offset 0)
			EmitBinaryConstOp(Context.PreshaderData, Op, Type, Dimension, 0, 0, InputB.GetData(Type), bScalarA, bScalarB);

			Context.SetInput(0, Type, bScalarA ? 1 : Dimension, InputA);
			Context.EvaluateTest(ExpectedOutput, Type, Dimension);

			// Reuse result doesn't work if first argument is scalar
			if (bScalarA == false)
			{
				// Test of reused result offset (previous op writes a constant of the given type to offset 0, then that value is used as the input AND result of the next op)
				EmitConstantOp(Context.PreshaderData, Type, bScalarA ? 1 : Dimension, 0, InputA.GetData(Type));
				EmitBinaryConstOp(Context.PreshaderData, Op, Type, Dimension, 0, 0, InputB.GetData(Type), bScalarA, bScalarB, true);

				Context.EvaluateTest(ExpectedOutput, Type, Dimension);
			}
		}
	}
}

static void UnitTestCast()
{
	FUnitTestContext Context(EPreshader2Opcode::Cast);
	FUnitTestValue Input
	{
		FVector4f(10000000, 10, 100, 1000),
		FVector4(10000000, 10, 100, 1000),
		{ 10000000, 10, 100, 1000 },
	};
	Input.InitLWCData();

	for (int32 Dimension = 1; Dimension <= 4; ++Dimension)
	{
		for (int32 DestTypeIndex = 0; DestTypeIndex <= (int32)EPreshader2Type::LWC; ++DestTypeIndex)
		{
			EPreshader2Type DestType = (EPreshader2Type)DestTypeIndex;
			int32 DestDataSize = Dimension * (DestType == EPreshader2Type::LWC ? 8 : FUnitTestValue::GetElementSize(DestType));

			for (int32 SrcTypeIndex = 0; SrcTypeIndex <= (int32)EPreshader2Type::Integer; ++SrcTypeIndex)
			{
				EPreshader2Type SrcType = (EPreshader2Type)SrcTypeIndex;

				// Casts to the same type aren't valid.
				if (SrcType == DestType)
				{
					continue;
				}

				EmitUnaryOp(Context.PreshaderData, EPreshader2Opcode::Cast, SrcType, Dimension, 0, 8, false, DestType);

				Context.SetInput(8, SrcType, Dimension, Input);
				Context.EvaluateTest(Input, DestType, Dimension);

				// Test of in-place result offset (data starts at offset 0, result written to offset 0)
				EmitUnaryOp(Context.PreshaderData, EPreshader2Opcode::Cast, SrcType, Dimension, 0, 0, false, DestType);

				// In place conversion to smaller type won't leave unused pattern in remaining bytes, so only do unused test if dest type isn't larger than source type.
				bool bCheckUnused = FUnitTestValue::GetElementSize(SrcType) <= FUnitTestValue::GetElementSize(DestType);
				Context.SetInput(0, SrcType, Dimension, Input);
				Context.EvaluateTest(Input, DestType, Dimension, Dimension, bCheckUnused);

				// Test of reused result offset (previous op writes a constant of the given type to offset 0, then that value is used as the input AND result of the next op)
				EmitConstantOp(Context.PreshaderData, SrcType, Dimension, 0, Input.GetData(SrcType));
				EmitUnaryOp(Context.PreshaderData, EPreshader2Opcode::Cast, SrcType, Dimension, 0, 0, true, DestType);

				Context.EvaluateTest(Input, DestType, Dimension, Dimension, bCheckUnused);
			}
		}
	}
}

static void UnitTestLength()
{
	FUnitTestContext Context(EPreshader2Opcode::Length);
	FUnitTestValue Input
	{
		FVector4f(10, 20, 30, 40),
		FVector4(10, 20, 30, 40),
	};
	FUnitTestValue ExpectedResult;

	// Output of length is a scalar.
	const int32 OutputDimension = 1;

	for (int32 Dimension = 2; Dimension <= 4; ++Dimension)
	{
		switch (Dimension)
		{
		case 2:
			ExpectedResult.Float[0] = FVector2f(Input.Float).Length();
			ExpectedResult.Double[0] = FVector2d(Input.Double).Length();
			break;
		case 3:
			ExpectedResult.Float[0] = FVector3f(Input.Float).Length();
			ExpectedResult.Double[0] = FVector3d(Input.Double).Length();
			break;
		case 4:
			ExpectedResult.Float[0] = FMath::Sqrt(Dot4(Input.Float, Input.Float));
			ExpectedResult.Double[0] = FMath::Sqrt(Dot4(Input.Double, Input.Double));
			break;
		}

		for (int32 TypeIndex = 0; TypeIndex <= 1; ++TypeIndex)
		{
			EPreshader2Type Type = (EPreshader2Type)TypeIndex;

			EmitUnaryOp(Context.PreshaderData, EPreshader2Opcode::Length, Type, Dimension, 0, 8, false);

			Context.SetInput(8, Type, Dimension, Input);
			Context.EvaluateTest(ExpectedResult, Type, Dimension, OutputDimension);

			// Test of in-place result offset (data starts at offset 0, result written to offset 0)
			EmitUnaryOp(Context.PreshaderData, EPreshader2Opcode::Length, Type, Dimension, 0, 0, false);

			Context.SetInput(0, Type, Dimension, Input);
			Context.EvaluateTest(ExpectedResult, Type, Dimension, OutputDimension, /*bCheckUnused=*/ false);		// Output type is smaller, don't check unused bytes
		}
	}
}

static void UnitTestNormalize()
{
	FUnitTestContext Context(EPreshader2Opcode::Normalize);
	FUnitTestValue Input
	{
		FVector4f(10, 20, 30, 40),
		FVector4(10, 20, 30, 40),
	};

	// Normalize downcasts to float automatically, so output type is always float
	const EPreshader2Type OutputType = EPreshader2Type::Float;

	for (int32 Dimension = 2; Dimension <= 4; ++Dimension)
	{
		for (int32 TypeIndex = 0; TypeIndex <= 1; ++TypeIndex)
		{
			EPreshader2Type Type = (EPreshader2Type)TypeIndex;

			// Generate output
			FUnitTestValue ExpectedOutput;
			if (Type == EPreshader2Type::Float)
			{
				switch (Dimension)
				{
				case 2:
					{
						FVector2f Input2(Input.Float);
						Input2.Normalize();
						ExpectedOutput.Float = FVector4f(Input2, FVector2f{});
						break;
					}
				case 3:
					{
						FVector3f Input3(Input.Float);
						Input3.Normalize();
						ExpectedOutput.Float = FVector4f(Input3, 0.0f);
						break;
					}
				default:
					{
						// No normalize implementation on vector4?  Use the quaternion version.
						FQuat4f TempQuat = FQuat4f(Input.Float.X, Input.Float.Y, Input.Float.Z, Input.Float.W);
						TempQuat.Normalize();
						ExpectedOutput.Float = FVector4f(TempQuat.X, TempQuat.Y, TempQuat.Z, TempQuat.W);
						break;
					}
				}
			}
			else
			{
				// Calculation is done with doubles, then downcasted
				switch (Dimension)
				{
				case 2:
					{
						FVector2d Input2(Input.Double);
						Input2.Normalize();
						ExpectedOutput.Float = FVector4f(FVector4(Input2, FVector2d{}));
						break;
					}
				case 3:
					{
						FVector3d Input3(Input.Double);
						Input3.Normalize();
						ExpectedOutput.Float = FVector4f(FVector4(Input3, 0.0));
						break;
					}
				default:
					{
						// No normalize implementation on vector4?  Use the quaternion version.
						FQuat4d TempQuat = FQuat4d(Input.Double.X, Input.Double.Y, Input.Double.Z, Input.Double.W);
						TempQuat.Normalize();
						ExpectedOutput.Float = FVector4f(FVector4(TempQuat.X, TempQuat.Y, TempQuat.Z, TempQuat.W));
						break;
					}
				}
			}

			// Test of separate source and destination offset (data starts at offset 8, result written to offset 0)
			EmitUnaryOp(Context.PreshaderData, EPreshader2Opcode::Normalize, Type, Dimension, 0, 8, false);

			Context.SetInput(8, Type, Dimension, Input);
			Context.EvaluateTest(ExpectedOutput, OutputType, Dimension);

			// Test of in-place result offset (data starts at offset 0, result written to offset 0)
			EmitUnaryOp(Context.PreshaderData, EPreshader2Opcode::Normalize, Type, Dimension, 0, 0, false);

			Context.SetInput(0, Type, Dimension, Input);
			Context.EvaluateTest(ExpectedOutput, OutputType, Dimension, Dimension, /*bCheckUnused=*/ Type == OutputType);
		}
	}
}

static void UnitTestCross(FUnitTestValue& InputA, FUnitTestValue& InputB)
{
	FUnitTestContext Context(EPreshader2Opcode::Cross);
	FUnitTestValue ExpectedOutput;
	ExpectedOutput.Float = InputA.Float ^ InputB.Float;
	ExpectedOutput.Double = InputA.Double ^ InputB.Double;

	for (int32 TypeIndex = 0; TypeIndex < 2; ++TypeIndex)
	{
		// Cross only operates on 3 component vectors
		const int32 Dimension = 3;
		EPreshader2Type Type = (EPreshader2Type)TypeIndex;

		// Test of separate source and destination offset (data starts at offset 8 and 16, result written to offset 0)
		EmitBinaryOp(Context.PreshaderData, EPreshader2Opcode::Cross, Type, Dimension, 0, 8, 16, false, false);

		Context.SetInput(8, Type, Dimension, InputA);
		Context.SetInput(16, Type, Dimension, InputB);
		Context.EvaluateTest(ExpectedOutput, Type, Dimension);

		// Test of in-place result offset (data starts at offset 0 and 16, result written to offset 0)
		EmitBinaryOp(Context.PreshaderData, EPreshader2Opcode::Cross, Type, Dimension, 0, 0, 16, false, false);

		Context.SetInput(0, Type, Dimension, InputA);
		Context.SetInput(16, Type, Dimension, InputB);
		Context.EvaluateTest(ExpectedOutput, Type, Dimension);
	}
}

static void UnitTestDot(FUnitTestValue& InputA, FUnitTestValue& InputB)
{
	FUnitTestContext Context(EPreshader2Opcode::Dot);

	// Output of dot is a scalar.
	const int32 OutputDimension = 1;

	for (int32 Dimension = 1; Dimension <= 4; ++Dimension)
	{
		FUnitTestValue ExpectedOutput;
		switch (Dimension)
		{
		case 1:
			ExpectedOutput.Float = FVector4f(InputA.Float[0] * InputB.Float[0]);
			ExpectedOutput.Double = FVector4(InputA.Double[0] * InputB.Double[0]);
			break;
		case 2:
			ExpectedOutput.Float = FVector4f(InputA.Float[0] * InputB.Float[0] + InputA.Float[1] * InputB.Float[1]);
			ExpectedOutput.Double = FVector4(InputA.Double[0] * InputB.Double[0] + InputA.Double[1] * InputB.Double[1]);
			break;
		case 3:
			ExpectedOutput.Float = FVector4f(Dot3(InputA.Float, InputB.Float));
			ExpectedOutput.Double = FVector4(Dot3(InputA.Double, InputB.Double));
			break;
		default:
			ExpectedOutput.Float = FVector4f(Dot4(InputA.Float, InputB.Float));
			ExpectedOutput.Double = FVector4(Dot4(InputA.Double, InputB.Double));
			break;
		}

		for (int32 TypeIndex = 0; TypeIndex < 2; ++TypeIndex)
		{
			EPreshader2Type Type = (EPreshader2Type)TypeIndex;

			// Test of separate source and destination offset (data starts at offset 8 and 16, result written to offset 0)
			EmitBinaryOp(Context.PreshaderData, EPreshader2Opcode::Dot, Type, Dimension, 0, 8, 16, false, false);

			Context.SetInput(8, Type, Dimension, InputA);
			Context.SetInput(16, Type, Dimension, InputB);
			Context.EvaluateTest(ExpectedOutput, Type, Dimension, OutputDimension);

			// Test of in-place result offset (data starts at offset 0 and 16, result written to offset 0)
			EmitBinaryOp(Context.PreshaderData, EPreshader2Opcode::Dot, Type, Dimension, 0, 0, 16, false, false);

			Context.SetInput(0, Type, Dimension, InputA);
			Context.SetInput(16, Type, Dimension, InputB);
			Context.EvaluateTest(ExpectedOutput, Type, Dimension, OutputDimension, /*bCheckUnused=*/ false);
		}
	}
}

// Move doesn't use FUnitTestContext, as it requires custom input initialization and validation.
static void UnitTestMove(TConstArrayView<uint16> Pattern)
{
	int32 Dimension = Pattern.Num();

	TStaticArray<float, 32> ResultsBuffer;
	UE::Shader::FPreshaderData PreshaderData;

	const uint8 UnusedByte = 0xeb;
	uint64 UnusedMemoryPattern;
	FMemory::Memset(UnusedMemoryPattern, UnusedByte);

	int32 DataSize;
	FUnitTestValue ExpectedOutput;
	FMemory::Memset(ExpectedOutput, UnusedByte);

	EPreshader2Type Type = EPreshader2Type::Float;

	FMemory::Memset(ResultsBuffer, UnusedByte);

	// Write the value in the pattern as float at the given offset.  This will produce an output equal to the input pattern
	// if the correct words are moved.
	for (int32 Index = 0; Index < Pattern.Num(); ++Index)
	{
		AsFloat(ResultsBuffer.GetData(), Pattern[Index], 0) = Pattern[Index];
		ExpectedOutput.Float[Index] = Pattern[Index];
	}

	{
		PreshaderData.Data.Empty();
		EmitMoveOp(PreshaderData, Type, 0, Pattern);

		UE::Shader::FPreshaderDataContext DataContext(PreshaderData, ResultsBuffer);
		Evaluate(DataContext);

		DataSize = Pattern.Num() * FUnitTestValue::GetElementSize(Type);
		check(FMemory::Memcmp(ResultsBuffer.GetData(), ExpectedOutput.GetData(Type), DataSize) == 0);

		// Check that memory after the destination of the move wasn't touched
		for (uint16 Index = Pattern.Num(); Index < 8; ++Index)
		{
			if (Pattern.Contains(Index))
			{
				// Memory was source data from the pattern, make sure it still has a value equal to its offset
				check(AsFloat(ResultsBuffer.GetData(), Index, 0) == (float)Index);
			}
			else
			{
				// Memory should be unused
				check(FMemory::Memcmp(&ResultsBuffer[Index], &UnusedMemoryPattern, sizeof(float)) == 0);
			}
		}
	}

	{
		// Repeat test for double format.  We'll double the offsets in the pattern since double format requires even offsets.
		TArray<uint16> PatternDouble;
		for (uint16 PatternOffset : Pattern)
		{
			PatternDouble.Add(PatternOffset * 2);
		}

		Type = EPreshader2Type::Double;

		FMemory::Memset(ResultsBuffer, UnusedByte);

		for (int32 Index = 0; Index < PatternDouble.Num(); ++Index)
		{
			AsDouble(ResultsBuffer.GetData(), PatternDouble[Index], 0) = PatternDouble[Index];
			ExpectedOutput.Double[Index] = PatternDouble[Index];
		}

		PreshaderData.Data.Empty();
		EmitMoveOp(PreshaderData, Type, 0, PatternDouble);

		UE::Shader::FPreshaderDataContext DataContext(PreshaderData, ResultsBuffer);
		Evaluate(DataContext);

		DataSize = PatternDouble.Num() * FUnitTestValue::GetElementSize(Type);
		check(FMemory::Memcmp(ResultsBuffer.GetData(), ExpectedOutput.GetData(Type), DataSize) == 0);

		// Check that memory after the destination of the move wasn't touched.  Check doubles at every other offset.
		for (uint16 Index = PatternDouble.Num() * 2; Index < 16; Index += 2)
		{
			if (PatternDouble.Contains(Index))
			{
				// Memory was source data from the pattern, make sure it still has a value equal to its offset
				check(AsDouble(ResultsBuffer.GetData(), Index, 0) == (double)Index);
			}
			else
			{
				// Memory should be unused
				check(FMemory::Memcmp(&ResultsBuffer[Index], &UnusedMemoryPattern, sizeof(double)) == 0);
			}
		}
	}
}

#define UNIT_TEST_UNARY(X,Y,Z,W,OP,FUNC) do {												\
	FUnitTestValue Input { FVector4f(X, Y, Z, W), FVector4(X, Y, Z, W), { X, Y, Z, W } };	\
	FUnitTestValue ExpectedOutput{															\
		FVector4f(FUNC((float)X), FUNC((float)Y), FUNC((float)Z), FUNC((float)W)),			\
		FVector4(FUNC((double)X), FUNC((double)Y), FUNC((double)Z), FUNC((double)W)),		\
		{ FUNC((int32)X), FUNC((int32)Y), FUNC((int32)Z), FUNC((int32)W) } };				\
	UnitTestUnaryOp(EPreshader2Opcode::OP, Input, ExpectedOutput); } while(0)

// Need to use volatile temporary variables to prevent the compiler from precomputing math function results, which can return
// different values in the last bit or two from runtime versions of the same functions, breaking our binary comparison.
#define UNIT_TEST_UNARY_NOINT(X,Y,Z,W,OP,FUNC) do { \
	volatile float XF=(float)X, YF=(float)Y, ZF=(float)Z, WF=(float)W;						\
	volatile double XD=(double)X, YD=(double)Y, ZD=(double)Z, WD=(double)W;					\
	FUnitTestValue Input { FVector4f(X, Y, Z, W), FVector4(X, Y, Z, W) };					\
	FUnitTestValue ExpectedOutput{															\
		FVector4f(FUNC(XF), FUNC(YF), FUNC(ZF), FUNC(WF)),									\
		FVector4(FUNC(XD), FUNC(YD), FUNC(ZD), FUNC(WD)) };									\
	UnitTestUnaryOp(EPreshader2Opcode::OP, Input, ExpectedOutput); } while (0)

#define UNIT_TEST_BINARY_SCALAR_PERMUTATION(A,B,C,D,X,Y,Z,W,OP,FUNC,SCALARA,SCALARB) do {					\
	volatile float AF=(float)A, BF=(float)B, CF=(float)C, DF=(float)D;										\
	volatile double AD=(double)A, BD=(double)B, CD=(double)C, DD=(double)D;									\
	volatile float XF=(float)X, YF=(float)Y, ZF=(float)Z, WF=(float)W;										\
	volatile double XD=(double)X, YD=(double)Y, ZD=(double)Z, WD=(double)W;									\
	FUnitTestValue InputA { FVector4f(A, B, C, D), FVector4(A, B, C, D), { A, B, C, D } };					\
	FUnitTestValue InputB { FVector4f(X, Y, Z, W), FVector4(X, Y, Z, W), { X, Y, Z, W } };					\
	FUnitTestValue ExpectedOutput{																			\
		FVector4f(FUNC(AF,XF),																				\
			FUNC(SCALARA ? AF : BF, SCALARB ? XF : YF),														\
			FUNC(SCALARA ? AF : CF, SCALARB ? XF : ZF),														\
			FUNC(SCALARA ? AF : DF, SCALARB ? XF : WF)),													\
		FVector4(FUNC(AD,XD),																				\
			FUNC(SCALARA ? AD : BD, SCALARB ? XD : YD),														\
			FUNC(SCALARA ? AD : CD, SCALARB ? XD : ZD),														\
			FUNC(SCALARA ? AD : DD, SCALARB ? XD : WD)),													\
		{ FUNC((int32)A,(int32)X),																			\
			FUNC((int32)(SCALARA ? A : B), (int32)(SCALARB ? X : Y)),										\
			FUNC((int32)(SCALARA ? A : C), (int32)(SCALARB ? X : Z)),										\
			FUNC((int32)(SCALARA ? A : D), (int32)(SCALARB ? X : W)) } };									\
	UnitTestBinaryOp(EPreshader2Opcode::OP, InputA, InputB, SCALARA, SCALARB, ExpectedOutput); } while(0)

#define UNIT_TEST_BINARY(A,B,C,D,X,Y,Z,W,OP,FUNC) \
	UNIT_TEST_BINARY_SCALAR_PERMUTATION(A,B,C,D,X,Y,Z,W,OP,FUNC, false, false);								\
	UNIT_TEST_BINARY_SCALAR_PERMUTATION(A,B,C,D,X,Y,Z,W,OP,FUNC, true, false);								\
	UNIT_TEST_BINARY_SCALAR_PERMUTATION(A,B,C,D,X,Y,Z,W,OP,FUNC, false, true);								\
	UNIT_TEST_BINARY_SCALAR_PERMUTATION(A,B,C,D,X,Y,Z,W,OP,FUNC, true, true)

#define UNIT_TEST_BINARY_NOINT_SCALAR_PERMUTATION(A,B,C,D,X,Y,Z,W,OP,FUNC,SCALARA,SCALARB) do {				\
	volatile float AF=(float)A, BF=(float)B, CF=(float)C, DF=(float)D;										\
	volatile double AD=(double)A, BD=(double)B, CD=(double)C, DD=(double)D;									\
	volatile float XF=(float)X, YF=(float)Y, ZF=(float)Z, WF=(float)W;										\
	volatile double XD=(double)X, YD=(double)Y, ZD=(double)Z, WD=(double)W;									\
	FUnitTestValue InputA { FVector4f(A, B, C, D), FVector4(A, B, C, D) };									\
	FUnitTestValue InputB { FVector4f(X, Y, Z, W), FVector4(X, Y, Z, W) };									\
	FUnitTestValue ExpectedOutput{																			\
		FVector4f(FUNC(AF,XF),																				\
			FUNC(SCALARA ? AF : BF, SCALARB ? XF : YF),														\
			FUNC(SCALARA ? AF : CF, SCALARB ? XF : ZF),														\
			FUNC(SCALARA ? AF : DF, SCALARB ? XF : WF)),													\
		FVector4(FUNC(AD,XD),																				\
			FUNC(SCALARA ? AD : BD, SCALARB ? XD : YD),														\
			FUNC(SCALARA ? AD : CD, SCALARB ? XD : ZD),														\
			FUNC(SCALARA ? AD : DD, SCALARB ? XD : WD)) };													\
	UnitTestBinaryOp(EPreshader2Opcode::OP, InputA, InputB, SCALARA, SCALARB, ExpectedOutput); } while(0)

#define UNIT_TEST_BINARY_NOINT(A,B,C,D,X,Y,Z,W,OP,FUNC) \
	UNIT_TEST_BINARY_NOINT_SCALAR_PERMUTATION(A,B,C,D,X,Y,Z,W,OP,FUNC, false, false);						\
	UNIT_TEST_BINARY_NOINT_SCALAR_PERMUTATION(A,B,C,D,X,Y,Z,W,OP,FUNC, true, false);						\
	UNIT_TEST_BINARY_NOINT_SCALAR_PERMUTATION(A,B,C,D,X,Y,Z,W,OP,FUNC, false, true);						\
	UNIT_TEST_BINARY_NOINT_SCALAR_PERMUTATION(A,B,C,D,X,Y,Z,W,OP,FUNC, true, true)

#define UNIT_TEST_BINARY_CROSS(A,B,C,X,Y,Z) do {												\
	FUnitTestValue InputA { FVector4f(A, B, C, 0.0f), FVector4(A, B, C, 0.0) };					\
	FUnitTestValue InputB { FVector4f(X, Y, Z, 0.0f), FVector4(X, Y, Z, 0.0) };					\
	UnitTestCross(InputA, InputB); } while(0)

#define UNIT_TEST_BINARY_DOT(A,B,C,D,X,Y,Z,W) do {												\
	FUnitTestValue InputA { FVector4f(A, B, C, D), FVector4(A, B, C, D) };						\
	FUnitTestValue InputB { FVector4f(X, Y, Z, W), FVector4(X, Y, Z, W) };						\
	UnitTestDot(InputA, InputB); } while(0)

//-V:UNIT_TEST:567
void UnitTest()
{
	UNIT_TEST_UNARY(-1, 2, -3, 4, Abs, FMath::Abs);
	UNIT_TEST_UNARY(5, -6, 7, -8, Abs, FMath::Abs);
	UNIT_TEST_UNARY(-1, 2, -3, 4, Negate, FOpNegate::Call);
	UNIT_TEST_UNARY(5, -6, 7, -8, Negate, FOpNegate::Call);
	UNIT_TEST_UNARY(-1, 0, 1, 2, Saturate, FOpSaturate::Call);
	UNIT_TEST_UNARY(2, 1, 0, -1, Saturate, FOpSaturate::Call);
	UNIT_TEST_UNARY_NOINT(-0.125, 0.125, -0.875, 0.875, Saturate, FOpSaturate::Call);
	UNIT_TEST_UNARY_NOINT(0.75, 1.25, 0.75, 1.25, Saturate, FOpSaturate::Call);
	UNIT_TEST_UNARY(-1, 2, -3, 4, Sign, FMath::Sign);
	UNIT_TEST_UNARY(5, -6, 7, -8, Sign, FMath::Sign);

	UNIT_TEST_UNARY_NOINT(-1.25, -1.0, -0.75, -0.5, ACos, FMath::Acos);
	UNIT_TEST_UNARY_NOINT(-0.25, 0.0, 0.25, 0.5, ACos, FMath::Acos);
	UNIT_TEST_UNARY_NOINT(0.5, 0.75, 1.0, 1.25, ACos, FMath::Acos);
	UNIT_TEST_UNARY_NOINT(-1.25, -1.0, -0.75, -0.5, ASin, FMath::Asin);
	UNIT_TEST_UNARY_NOINT(-0.25, 0.0, 0.25, 0.5, ASin, FMath::Asin);
	UNIT_TEST_UNARY_NOINT(0.5, 0.75, 1.0, 1.25, ASin, FMath::Asin);
	UNIT_TEST_UNARY_NOINT(-1.25, -1.0, -0.75, -0.5, ATan, FMath::Atan);
	UNIT_TEST_UNARY_NOINT(-0.25, 0.0, 0.25, 0.5, ATan, FMath::Atan);
	UNIT_TEST_UNARY_NOINT(0.5, 0.75, 1.0, 1.25, ATan, FMath::Atan);
	UNIT_TEST_UNARY_NOINT(-0.125, 0.125, -0.875, 0.875, Ceil, FMath::CeilToFloat);
	UNIT_TEST_UNARY_NOINT(0.75, 1.25, 0.75, 1.25, Ceil, FMath::CeilToFloat);
	UNIT_TEST_UNARY_NOINT(1, 2, 3, 4, Cos, FMath::Cos);
	UNIT_TEST_UNARY_NOINT(1, 2, 3, 4, Exponential, FMath::Exp);
	UNIT_TEST_UNARY_NOINT(1, 2, 3, 4, Exponential2, FMath::Exp2);
	UNIT_TEST_UNARY_NOINT(-0.125, 0.125, -0.875, 0.875, Floor, FMath::FloorToFloat);
	UNIT_TEST_UNARY_NOINT(0.75, 1.25, 0.75, 1.25, Floor, FMath::FloorToFloat);
	UNIT_TEST_UNARY_NOINT(-0.125, 0.125, -0.875, 0.875, Frac, FMath::Frac);
	UNIT_TEST_UNARY_NOINT(0.75, 1.25, 0.75, 1.25, Frac, FMath::Frac);
	UNIT_TEST_UNARY_NOINT(-0.125, 0.125, -0.875, 0.875, Fractional, FMath::Fractional);
	UNIT_TEST_UNARY_NOINT(0.75, 1.25, 0.75, 1.25, Fractional, FMath::Fractional);
	UNIT_TEST_UNARY_NOINT(0.5, 1.0, 1.5, 2.0, Logarithm, FMath::Loge);
	UNIT_TEST_UNARY_NOINT(0.5, 1.0, 1.5, 2.0, Logarithm10, FOpLogarithm10::Call);
	UNIT_TEST_UNARY_NOINT(0.5, 1.0, 1.5, 2.0, Logarithm2, FMath::Log2);
	UNIT_TEST_UNARY_NOINT(-0.125, 0.125, -0.875, 0.875, Reciprocal, FOpReciprocal::Call);
	UNIT_TEST_UNARY_NOINT(-0.000000000000001f, 0.0f, 0.000000000000001f, 2.0f, Reciprocal, FOpReciprocal::Call);
	UNIT_TEST_UNARY_NOINT(-0.125, 0.125, -0.875, 0.875, Round, FMath::RoundToFloat);
	UNIT_TEST_UNARY_NOINT(0.75, 1.25, 0.75, 1.25, Round, FMath::RoundToFloat);
	UNIT_TEST_UNARY_NOINT(1, 2, 3, 4, Sin, FMath::Sin);
	UNIT_TEST_UNARY_NOINT(1, 2, 3, 4, Sqrt, FMath::Sqrt);
	UNIT_TEST_UNARY_NOINT(1, 2, 3, 4, Tan, FMath::Tan);
	UNIT_TEST_UNARY_NOINT(-0.125, 0.125, -0.875, 0.875, Truncate, FMath::TruncToFloat);
	UNIT_TEST_UNARY_NOINT(0.75, 1.25, 0.75, 1.25, Truncate, FMath::TruncToFloat);

	UNIT_TEST_BINARY(2, 3, 40, 50, 60, 70, 8, 9, Add, FOpAdd::Call);
	UNIT_TEST_BINARY(2, 3, 40, 50, 60, 70, 8, 9, Divide, FOpDivide::Call);
	UNIT_TEST_BINARY(2, 3, 40, 50, 60, 70, 8, 9, Fmod, FOpFmod::Call);
	UNIT_TEST_BINARY(2, 3, 40, 50, 60, 70, 8, 9, Max, FMath::Max);
	UNIT_TEST_BINARY(2, 3, 40, 50, 60, 70, 8, 9, Min, FMath::Min);
	UNIT_TEST_BINARY(2, 3, 40, 50, 60, 70, 8, 9, Multiply, FOpMultiply::Call);
	UNIT_TEST_BINARY(2, 3, 40, 50, 60, 70, 8, 9, Subtract, FOpSubtract::Call);

	UNIT_TEST_BINARY_NOINT(2, 3, 40, 50, 60, 70, 8, 9, ATan2, FMath::Atan2);

	UNIT_TEST_BINARY_CROSS(1, -2, 3, -4, 5, -6);
	UNIT_TEST_BINARY_DOT(0.5, 1.0, 1.5, 2.0, 2.5, 3.0, 3.5, 4.0);

	UnitTestCast();
	UnitTestLength();
	UnitTestNormalize();

	// Exhaustively test a bunch of move permutations
	for (uint16 IndexA = 0; IndexA < 8; ++IndexA)
	{
		// Permutations with one input
		UnitTestMove({ IndexA });

		for (uint16 IndexB = 0; IndexB < 8; ++IndexB)
		{
			// Permutations with two inputs
			UnitTestMove({ IndexA, IndexB });

			for (uint16 IndexC = 0; IndexC < 8; ++IndexC)
			{
				// Permutations with three inputs
				UnitTestMove({ IndexA, IndexB, IndexC });

				for (uint16 IndexD = 0; IndexD < 8; ++IndexD)
				{
					// Permutations with four inputs
					UnitTestMove({ IndexA, IndexB, IndexC, IndexD });
				}
			}
		}
	}

	// Note that "Constant" op is unit tested by unary and binary ops when testing "Reuse" opcode variants,
	// so we don't need a separate unit test for that.
}

}  // namespace UE::Preshader2