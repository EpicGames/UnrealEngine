// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Shader/Preshader.h"

class FUniformExpressionSet;
struct FMaterialRenderContext;

namespace UE
{
namespace Shader
{

class FPreshaderData;

struct FPreshaderDataContext
{
	explicit FPreshaderDataContext(const FPreshaderData& InData, TArrayView<float> InResultsBuffer = {});
	FPreshaderDataContext(const FPreshaderDataContext& InContext, uint32 InOffset, uint32 InSize);

	const uint8* RESTRICT Ptr;
	const uint8* RESTRICT EndPtr;
	TArrayView<const FScriptName> Names;
	TArrayView<float> ResultsBuffer;				// Used by Preshader2 for results and intermediates

	bool IsPreshader2() const { return !ResultsBuffer.IsEmpty(); }
};

ENGINE_API FPreshaderValue EvaluatePreshader(const FUniformExpressionSet* UniformExpressionSet, const FMaterialRenderContext& Context, FPreshaderStack& Stack, FPreshaderDataContext& RESTRICT Data);

ENGINE_API FString PreshaderGenerateDebugString(const FUniformExpressionSet& UniformExpressionSet, const FMaterialRenderContext& Context, FPreshaderDataContext& RESTRICT Data, TMap<FString, uint32>* ParameterReferences = nullptr);
ENGINE_API void PreshaderComputeDebugStats(const FUniformExpressionSet& UniformExpressionSet, const FMaterialRenderContext& Context, FPreshaderDataContext& RESTRICT Data, uint32& TotalParameters, uint32& TotalOps);

} // namespace Shader
} // namespace UE
