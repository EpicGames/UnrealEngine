// Copyright Epic Games, Inc. All Rights Reserved.
#include "Shader/ShaderTypes.h"
#include "Hash/xxhash.h"
#include "Misc/MemStackUtility.h"

namespace UE
{
namespace Shader
{

const FValueTypeDescription GValueTypeDescriptions[] =
{
	{ TEXT("void"),			EValueType::Void,		EValueComponentType::Void,		0, 0 },
	{ TEXT("float"),		EValueType::Float1,		EValueComponentType::Float,		1, sizeof(float) },
	{ TEXT("float2"),		EValueType::Float2,		EValueComponentType::Float,		2, sizeof(float) },
	{ TEXT("float3"),		EValueType::Float3,		EValueComponentType::Float,		3, sizeof(float) },
	{ TEXT("float4"),		EValueType::Float4,		EValueComponentType::Float,		4, sizeof(float) },
	{ TEXT("FWSScalar"),	EValueType::Double1,	EValueComponentType::Double,	1, sizeof(double) },
	{ TEXT("FWSVector2"),	EValueType::Double2,	EValueComponentType::Double,	2, sizeof(double) },
	{ TEXT("FWSVector3"),	EValueType::Double3,	EValueComponentType::Double,	3, sizeof(double) },
	{ TEXT("FWSVector4"),	EValueType::Double4,	EValueComponentType::Double,	4, sizeof(double) },
	{ TEXT("int"),			EValueType::Int1,		EValueComponentType::Int,		1, sizeof(int32) },
	{ TEXT("int2"),			EValueType::Int2,		EValueComponentType::Int,		2, sizeof(int32) },
	{ TEXT("int3"),			EValueType::Int3,		EValueComponentType::Int,		3, sizeof(int32) },
	{ TEXT("int4"),			EValueType::Int4,		EValueComponentType::Int,		4, sizeof(int32) },
	{ TEXT("bool"),			EValueType::Bool1,		EValueComponentType::Bool,		1, 1 },
	{ TEXT("bool2"),		EValueType::Bool2,		EValueComponentType::Bool,		2, 1 },
	{ TEXT("bool3"),		EValueType::Bool3,		EValueComponentType::Bool,		3, 1 },
	{ TEXT("bool4"),		EValueType::Bool4,		EValueComponentType::Bool,		4, 1 },
	{ TEXT("Numeric1"),		EValueType::Numeric1,	EValueComponentType::Numeric,	1, sizeof(double) },
	{ TEXT("Numeric2"),		EValueType::Numeric2,	EValueComponentType::Numeric,	2, sizeof(double) },
	{ TEXT("Numeric3"),		EValueType::Numeric3,	EValueComponentType::Numeric,	3, sizeof(double) },
	{ TEXT("Numeric4"),		EValueType::Numeric4,	EValueComponentType::Numeric,	4, sizeof(double) },
	{ TEXT("float4x4"),		EValueType::Float4x4,	EValueComponentType::Float,		16, sizeof(float) },
	{ TEXT("FWSMatrix"),	EValueType::Double4x4,	EValueComponentType::Double,	16, sizeof(double) },
	{ TEXT("FWSInverseMatrix"), EValueType::DoubleInverse4x4, EValueComponentType::Double, 16, sizeof(double) },
	{ TEXT("Numeric4x4"),	EValueType::Numeric4x4, EValueComponentType::Numeric,	16, sizeof(double) },
	{ TEXT("struct"),		EValueType::Struct,		EValueComponentType::Void,		0, 0 },
	{ TEXT("object"),		EValueType::Object,		EValueComponentType::Void,		0, 0 },
	{ TEXT("Any"),			EValueType::Any,		EValueComponentType::Void,		0, 0 },
	{ TEXT("<INVALID>"),	EValueType::Num,		EValueComponentType::Void,		0, 0 },
};

static_assert(UE_ARRAY_COUNT(GValueTypeDescriptions) == (int32)EValueType::Num + 1, "Missing entry from shader value description table");

#if DO_CHECK
// Startup validation logic for table above
struct FValidateShaderValueTypeDescriptionTable
{
	FValidateShaderValueTypeDescriptionTable()
	{
		for (int32 DescriptionIndex = 0; DescriptionIndex <= (int32)EValueType::Num; DescriptionIndex++)
		{
			// Make sure element location in array matches its ValueType (validates that the array matches the enum order)
			check((int32)GValueTypeDescriptions[DescriptionIndex].ValueType == DescriptionIndex);

			// Make sure component size matches the component type
			check(GValueTypeDescriptions[DescriptionIndex].ComponentSizeInBytes == GetComponentTypeSizeInBytes(GValueTypeDescriptions[DescriptionIndex].ComponentType));
		}
	}
};

FValidateShaderValueTypeDescriptionTable GValidateShaderValueTypeDescriptionTable;
#endif  // DO_CHECK

const FValueTypeDescription& GetValueTypeDescription(EValueType Type)
{
	uint32 TypeIndex = (uint32)Type;
	check((uint32)Type < NumValueTypes);

	return GValueTypeDescriptions[FMath::Min(TypeIndex, (uint32)NumValueTypes)];
}

const TCHAR* FType::GetName() const
{
	if (IsStruct())
	{
		check(ValueType == EValueType::Struct);
		return StructType->Name;
	}

	// TODO - do we need specific object names?
	check(ValueType != EValueType::Struct);
	const FValueTypeDescription& TypeDesc = GetValueTypeDescription(ValueType);
	return TypeDesc.Name;
}

FType FType::GetDerivativeType() const
{
	if (IsStruct())
	{
		check(ValueType == EValueType::Struct);
		return StructType->DerivativeType; // will convert to 'Void' if DerivativeType is nullptr
	}
	else if (IsObject())
	{
		return *this;
	}

	check(ValueType != EValueType::Struct);
	return MakeDerivativeType(ValueType);
}

int32 FType::GetNumComponents() const
{
	if (IsStruct())
	{
		check(ValueType == EValueType::Struct);
		return StructType->ComponentTypes.Num();
	}
	else if (IsObject())
	{
		return 1;
	}

	check(ValueType != EValueType::Struct);
	const FValueTypeDescription& TypeDesc = GetValueTypeDescription(ValueType);
	return TypeDesc.NumComponents;
}

int32 FType::GetNumFlatFields() const
{
	if (IsStruct())
	{
		check(ValueType == EValueType::Struct);
		return StructType->FlatFieldTypes.Num();
	}

	check(ValueType != EValueType::Struct);
	return 1;
}

EValueComponentType FType::GetComponentType(int32 Index) const
{
	if (Index < 0)
	{
		return EValueComponentType::Void;
	}

	if (IsStruct())
	{
		check(ValueType == EValueType::Struct);
		if (Index < StructType->ComponentTypes.Num())
		{
			return StructType->ComponentTypes[Index];
		}
	}
	else if(IsNumeric())
	{
		const FValueTypeDescription& TypeDesc = GetValueTypeDescription(ValueType);
		// Scalar types replicate xyzw
		if ((TypeDesc.NumComponents == 1 && Index < 4) || Index < TypeDesc.NumComponents)
		{
			return TypeDesc.ComponentType;
		}
	}

	return EValueComponentType::Void;
}

EValueType FType::GetFlatFieldType(int32 Index) const
{
	if (IsStruct())
	{
		check(ValueType == EValueType::Struct);
		return StructType->FlatFieldTypes.IsValidIndex(Index) ? StructType->FlatFieldTypes[Index] : EValueType::Void;
	}
	else
	{
		return (Index == 0) ? ValueType : EValueType::Void;
	}
}

FType CombineTypes(const FType& Lhs, const FType& Rhs, bool bMergeMatrixTypes)
{
	if (Lhs.IsVoid() || Lhs.IsAny())
	{
		return Rhs;
	}
	if (Rhs.IsVoid() || Rhs.IsAny())
	{
		return Lhs;
	}

	if ((Lhs.IsNumericVector() && Rhs.IsNumericVector()) || (bMergeMatrixTypes && Lhs.IsNumericMatrix() && Rhs.IsNumericMatrix()))
	{
		const FValueTypeDescription& LhsDesc = GetValueTypeDescription(Lhs);
		const FValueTypeDescription& RhsDesc = GetValueTypeDescription(Rhs);
		const EValueComponentType ComponentType = CombineComponentTypes(LhsDesc.ComponentType, RhsDesc.ComponentType);
		if (ComponentType == EValueComponentType::Void)
		{
			return Shader::EValueType::Void;
		}

		const int8 NumComponents = FMath::Max(LhsDesc.NumComponents, RhsDesc.NumComponents);
		return MakeValueType(ComponentType, NumComponents);
	}

	// Non-numeric types can only combine if they are the same
	if (Lhs == Rhs)
	{
		return Lhs;
	}

	return Shader::EValueType::Void;
}

const FStructField* FStructType::FindFieldByName(const TCHAR* InName) const
{
	for (const FStructField& Field : Fields)
	{
		if (FCString::Strcmp(Field.Name, InName) == 0)
		{
			return &Field;
		}
	}

	return nullptr;
}

namespace Private
{
struct FCastFloat
{
	using FComponentType = float;
	inline float operator()(EValueComponentType Type, const FValueComponent& Component) const
	{
		switch (Type)
		{
		case EValueComponentType::Float: return Component.Float;
		case EValueComponentType::Double: return (float)Component.Double;
		case EValueComponentType::Int: return (float)Component.Int;
		case EValueComponentType::Bool: return (float)Component.Bool;
		default: return 0.0f;
		}
	}
};

struct FCastDouble
{
	using FComponentType = double;
	inline double operator()(EValueComponentType Type, const FValueComponent& Component) const
	{
		switch (Type)
		{
		case EValueComponentType::Float: return (double)Component.Float;
		case EValueComponentType::Double: return Component.Double;
		case EValueComponentType::Int: return (double)Component.Int;
		case EValueComponentType::Bool: return (double)Component.Bool;
		default: return 0.0f;
		}
	}
};

struct FCastInt
{
	using FComponentType = int32;
	inline int32 operator()(EValueComponentType Type, const FValueComponent& Component) const
	{
		switch (Type)
		{
		case EValueComponentType::Float: return (int32)Component.Float;
		case EValueComponentType::Double: return (int32)Component.Double;
		case EValueComponentType::Int: return Component.Int;
		case EValueComponentType::Bool: return Component.Bool ? 1 : 0;
		default: return false;
		}
	}
};

struct FCastBool
{
	using FComponentType = bool;
	inline bool operator()(EValueComponentType Type, const FValueComponent& Component) const
	{
		switch (Type)
		{
		case EValueComponentType::Float: return Component.Float != 0.0f;
		case EValueComponentType::Double: return Component.Double != 0.0;
		case EValueComponentType::Int: return Component.Int != 0;
		case EValueComponentType::Bool: return Component.AsBool();
		default: return false;
		}
	}
};

template<typename Operation, typename ResultType>
void AsType(const Operation& Op, const FValue& Value, ResultType& OutResult)
{
	using FComponentType = typename Operation::FComponentType;
	const FValueTypeDescription& TypeDesc = GetValueTypeDescription(Value.Type);
	if (TypeDesc.NumComponents == 1)
	{
		const FComponentType Component = Op(TypeDesc.ComponentType, Value.Component[0]);
		for (int32 i = 0; i < 4; ++i)
		{
			OutResult[i] = Component;
		}
	}
	else
	{
		const int32 NumComponents = FMath::Min<int32>(TypeDesc.NumComponents, 4);
		for (int32 i = 0; i < NumComponents; ++i)
		{
			OutResult[i] = Op(TypeDesc.ComponentType, Value.Component[i]);
		}
		for (int32 i = NumComponents; i < 4; ++i)
		{
			OutResult[i] = (FComponentType)0;
		}
	}
}

// Identical to "AsType" above, except Type and Component are loose, rather than pulled from an FValue structure
template<typename Operation, typename ResultType>
void AsTypeInPlace(const Operation& Op, EValueType Type, TArrayView<FValueComponent> Component, ResultType& OutResult)
{
	using FComponentType = typename Operation::FComponentType;
	const FValueTypeDescription& TypeDesc = GetValueTypeDescription(Type);
	if (TypeDesc.NumComponents == 1)
	{
		const FComponentType ComponentCast = Op(TypeDesc.ComponentType, Component[0]);
		for (int32 i = 0; i < 4; ++i)
		{
			OutResult[i] = ComponentCast;
		}
	}
	else
	{
		const int32 NumComponents = FMath::Min<int32>(TypeDesc.NumComponents, 4);
		for (int32 i = 0; i < NumComponents; ++i)
		{
			OutResult[i] = Op(TypeDesc.ComponentType, Component[i]);
		}
		for (int32 i = NumComponents; i < 4; ++i)
		{
			OutResult[i] = (FComponentType)0;
		}
	}
}

template<typename Operation>
void Cast(const Operation& Op, const FValue& Value, FValue& OutResult)
{
	using FComponentType = typename Operation::FComponentType;
	const FValueTypeDescription& ValueTypeDesc = GetValueTypeDescription(Value.Type);
	const FValueTypeDescription& ResultTypeDesc = GetValueTypeDescription(OutResult.Type);
	const int32 NumCopyComponents = FMath::Min<int32>(ValueTypeDesc.NumComponents, ResultTypeDesc.NumComponents);
	for (int32 i = 0; i < NumCopyComponents; ++i)
	{
		OutResult.Component.Add(Op(ValueTypeDesc.ComponentType, Value.Component[i]));
	}

	if (NumCopyComponents < ResultTypeDesc.NumComponents)
	{
		if (NumCopyComponents == 1)
		{
			const FValueComponent Component = OutResult.Component[0];
			for (int32 i = 1; i < ResultTypeDesc.NumComponents; ++i)
			{
				OutResult.Component.Add(Component);
			}
		}
		else
		{
			for (int32 i = NumCopyComponents; i < ResultTypeDesc.NumComponents; ++i)
			{
				OutResult.Component.AddDefaulted();
			}
		}
	}
}

void FormatComponent_Double(double Value, int32 NumComponents, EValueStringFormat Format, FStringBuilderBase& OutResult)
{
	if (Format == EValueStringFormat::HLSL)
	{
		OutResult.Appendf(TEXT("%0.8f"), Value);
	}
	else
	{
		// Shorter format for more components
		switch (NumComponents)
		{
		default: OutResult.Appendf(TEXT("%.2g"), Value); break;
		case 3: OutResult.Appendf(TEXT("%.3g"), Value); break;
		case 2: OutResult.Appendf(TEXT("%.3g"), Value); break;
		case 1: OutResult.Appendf(TEXT("%.4g"), Value); break;
		}
	}
}

} // namespace Private

FValue FValue::FromMemoryImage(EValueType Type, const void* Data, uint32* OutSizeInBytes)
{
	check(IsNumericType(Type));
	const FValueTypeDescription& TypeDesc = GetValueTypeDescription(Type);

	FValue Result(TypeDesc.ComponentType, TypeDesc.NumComponents);
	const uint8* Bytes = static_cast<const uint8*>(Data);
	const uint32 ComponentSizeInBytes = TypeDesc.ComponentSizeInBytes;
	if (ComponentSizeInBytes > 0u)
	{
		for (int32 i = 0u; i < TypeDesc.NumComponents; ++i)
		{
			FMemory::Memcpy(&Result.Component[i].Packed, Bytes, ComponentSizeInBytes);
			Bytes += ComponentSizeInBytes;
		}
	}
	if (OutSizeInBytes)
	{
		*OutSizeInBytes = (uint32)(Bytes - static_cast<const uint8*>(Data));
	}
	return Result;
}

FMemoryImageValue FValue::AsMemoryImage() const
{
	check(Type.IsNumeric());
	const FValueTypeDescription& TypeDesc = GetValueTypeDescription(Type);

	FMemoryImageValue Result;
	uint8* Bytes = Result.Bytes;
	const uint32 ComponentSizeInBytes = TypeDesc.ComponentSizeInBytes;
	if (ComponentSizeInBytes > 0u)
	{
		for (int32 i = 0u; i < TypeDesc.NumComponents; ++i)
		{
			FMemory::Memcpy(Bytes, &Component[i].Packed, ComponentSizeInBytes);
			Bytes += ComponentSizeInBytes;
		}
	}
	Result.Size = (uint32)(Bytes - Result.Bytes);
	check(Result.Size <= FMemoryImageValue::MaxSize);
	return Result;
}

FFloatValue FValue::AsFloat() const
{
	FFloatValue Result;
	Private::AsType(Private::FCastFloat(), *this, Result);
	return Result;
}

FDoubleValue FValue::AsDouble() const
{
	FDoubleValue Result;
	Private::AsType(Private::FCastDouble(), *this, Result);
	return Result;
}

FLinearColor FValue::AsLinearColor() const
{
	const FFloatValue Result = AsFloat();
	return FLinearColor(Result.Component[0], Result.Component[1], Result.Component[2], Result.Component[3]);
}

FVector4d FValue::AsVector4d() const
{
	const FDoubleValue Result = AsDouble();
	return FVector4d(Result.Component[0], Result.Component[1], Result.Component[2], Result.Component[3]);
}

FIntValue FValue::AsInt() const
{
	FIntValue Result;
	Private::AsType(Private::FCastInt(), *this, Result);
	return Result;
}

FBoolValue FValue::AsBool() const
{
	FBoolValue Result;
	Private::AsType(Private::FCastBool(), *this, Result);
	return Result;
}

float FValue::AsFloatScalar() const
{
	FFloatValue Result;
	Private::AsType(Private::FCastFloat(), *this, Result);
	return Result[0];
}

bool FValue::AsBoolScalar() const
{
	const FBoolValue Result = AsBool();
	for (int32 i = 0; i < 4; ++i)
	{
		if (Result.Component[i])
		{
			return true;
		}
	}
	return false;
}

bool FValue::IsZero() const
{
	bool bIsZero = Type.IsNumeric();
	if (bIsZero)
	{
		for (const FValueComponent& Comp : Component)
		{
			if (Comp.Packed)
			{
				bIsZero = false;
				break;
			}
		}
	}
	return bIsZero;
}

FValueComponentTypeDescription GetValueComponentTypeDescription(EValueComponentType Type)
{
	switch (Type)
	{
	case EValueComponentType::Void: return FValueComponentTypeDescription(TEXT("void"), 0u, EComponentBound::Zero, EComponentBound::Zero);
	case EValueComponentType::Float: return FValueComponentTypeDescription(TEXT("float"), sizeof(float), EComponentBound::NegFloatMax, EComponentBound::FloatMax);
	case EValueComponentType::Double: return FValueComponentTypeDescription(TEXT("double"), sizeof(double), EComponentBound::NegDoubleMax, EComponentBound::DoubleMax);
	case EValueComponentType::Int: return FValueComponentTypeDescription(TEXT("int"), sizeof(int32), EComponentBound::IntMin, EComponentBound::IntMax);
	case EValueComponentType::Bool: return FValueComponentTypeDescription(TEXT("bool"), 1u, EComponentBound::Zero, EComponentBound::One);
	case EValueComponentType::Numeric: return FValueComponentTypeDescription(TEXT("Numeric"), sizeof(double), EComponentBound::NegDoubleMax, EComponentBound::DoubleMax);
	default: checkNoEntry() return FValueComponentTypeDescription();
	}
}

EValueComponentType CombineComponentTypes(EValueComponentType Lhs, EValueComponentType Rhs)
{
	if (Lhs == Rhs)
	{
		return Lhs;
	}
	else if (Lhs == EValueComponentType::Void)
	{
		return Rhs;
	}
	else if (Rhs == EValueComponentType::Void)
	{
		return Lhs;
	}
	else if (Lhs == EValueComponentType::Numeric && Rhs == EValueComponentType::Numeric)
	{
		return EValueComponentType::Numeric;
	}
	else if (Lhs == EValueComponentType::Double || Rhs == EValueComponentType::Double)
	{
		return EValueComponentType::Double;
	}
	else if (Lhs == EValueComponentType::Float || Rhs == EValueComponentType::Float)
	{
		return EValueComponentType::Float;
	}
	else if(IsNumericType(Lhs) && IsNumericType(Rhs))
	{
		return EValueComponentType::Int;
	}
	else
	{
		return EValueComponentType::Void;
	}
}

const TCHAR* FValueComponent::ToString(EValueComponentType Type, FStringBuilderBase& OutString) const
{
	switch (Type)
	{
	case EValueComponentType::Int: OutString.Appendf(TEXT("%d"), Int); break;
	case EValueComponentType::Bool: OutString.Append(AsBool() ? TEXT("true") : TEXT("false")); break;
	case EValueComponentType::Float: OutString.Appendf(TEXT("%#.9gf"), Float); break;
	default: checkNoEntry(); break; // TODO - double, Numeric
	}
	return OutString.ToString();
}

const TCHAR* FValue::ToString(EValueStringFormat Format, FStringBuilderBase& OutString) const
{
	const int32 NumComponents = Type.GetNumComponents();
	const TCHAR* TypeName = Type.GetName();
	const TCHAR* ClosingSuffix = nullptr;

	if (Format == EValueStringFormat::HLSL)
	{
		if (Type.IsStruct())
		{
			OutString.Append(TEXT("{ "));
			ClosingSuffix = TEXT(" }");
		}
		else
		{
			const FValueTypeDescription& TypeDesc = GetValueTypeDescription(Type.ValueType);
			check(TypeDesc.ComponentType != EValueComponentType::Numeric);
			if (TypeDesc.ComponentType != EValueComponentType::Double)
			{
				OutString.Appendf(TEXT("%s("), TypeDesc.Name);
				ClosingSuffix = TEXT(")");
			}
		}
	}

	for (int32 Index = 0; Index < NumComponents; ++Index)
	{
		if (Index > 0)
		{
			OutString.Append(TEXT(", "));
		}
		const EValueComponentType ComponentType = Type.GetComponentType(Index);
		switch (ComponentType)
		{
		case EValueComponentType::Int: OutString.Appendf(TEXT("%d"), Component[Index].Int); break;
		case EValueComponentType::Bool: OutString.Append(Component[Index].Bool ? TEXT("true") : TEXT("false")); break;
		case EValueComponentType::Float: Private::FormatComponent_Double((double)Component[Index].Float, NumComponents, Format, OutString); break;
		case EValueComponentType::Double:
		case EValueComponentType::Numeric:
			Private::FormatComponent_Double(Component[Index].Double, NumComponents, Format, OutString); break;
		default: checkNoEntry(); break;
		}
	}

	if (ClosingSuffix)
	{
		OutString.Append(ClosingSuffix);
	}

	return OutString.ToString();
}

EValueType FindValueType(FName Name)
{
	for (int32 TypeIndex = 1; TypeIndex < NumValueTypes; ++TypeIndex)
	{
		const EValueType Type = (EValueType)TypeIndex;
		const FValueTypeDescription& TypeDesc = GetValueTypeDescription(Type);
		if (Name == TypeDesc.Name)
		{
			return Type;
		}
	}
	return EValueType::Void;
}

EValueType MakeValueType(EValueComponentType ComponentType, int32 NumComponents)
{
	if (ComponentType == EValueComponentType::Void || NumComponents == 0)
	{
		return EValueType::Void;
	}

	switch (ComponentType)
	{
	case EValueComponentType::Float:
		switch (NumComponents)
		{
		case 1: return EValueType::Float1;
		case 2: return EValueType::Float2;
		case 3: return EValueType::Float3;
		case 4: return EValueType::Float4;
		case 16: return EValueType::Float4x4;
		default: break;
		}
		break;
	case EValueComponentType::Double:
		switch (NumComponents)
		{
		case 1: return EValueType::Double1;
		case 2: return EValueType::Double2;
		case 3: return EValueType::Double3;
		case 4: return EValueType::Double4;
		case 16: return EValueType::Double4x4;
		default: break;
		}
		break;
	case EValueComponentType::Numeric:
		switch (NumComponents)
		{
		case 1: return EValueType::Numeric1;
		case 2: return EValueType::Numeric2;
		case 3: return EValueType::Numeric3;
		case 4: return EValueType::Numeric4;
		case 16: return EValueType::Numeric4x4;
		default: break;
		}
		break;
	case EValueComponentType::Int:
		switch (NumComponents)
		{
		case 1: return EValueType::Int1;
		case 2: return EValueType::Int2;
		case 3: return EValueType::Int3;
		case 4: return EValueType::Int4;
		case 16: return EValueType::Float4x4; // no explicit int/bool matrix types...so use float instead
		default: break;
		}
		break;
	case EValueComponentType::Bool:
		switch (NumComponents)
		{
		case 1: return EValueType::Bool1;
		case 2: return EValueType::Bool2;
		case 3: return EValueType::Bool3;
		case 4: return EValueType::Bool4;
		case 16: return EValueType::Float4x4;
		default: break;
		}
		break;
	default:
		break;
	}

	checkNoEntry();
	return EValueType::Void;
}

EValueType MakeValueType(EValueType BaseType, int32 NumComponents)
{
	return MakeValueType(GetValueTypeDescription(BaseType).ComponentType, NumComponents);
}

EValueType MakeValueTypeWithRequestedNumComponents(EValueType BaseType, int8 RequestedNumComponents)
{
	const FValueTypeDescription& TypeDesc = GetValueTypeDescription(BaseType);
	return MakeValueType(TypeDesc.ComponentType, FMath::Min(TypeDesc.NumComponents, RequestedNumComponents));
}

EValueType MakeNonLWCType(EValueType Type)
{
	const FValueTypeDescription& TypeDesc = GetValueTypeDescription(Type);
	check(IsNumericType(TypeDesc.ComponentType));
	if (TypeDesc.ComponentType == EValueComponentType::Double)
	{
		return MakeValueType(MakeNonLWCType(TypeDesc.ComponentType), TypeDesc.NumComponents);
	}
	return Type;
}

EValueType MakeConcreteType(EValueType Type)
{
	const FValueTypeDescription& TypeDesc = GetValueTypeDescription(Type);
	check(IsNumericType(TypeDesc.ComponentType));
	if (TypeDesc.ComponentType == EValueComponentType::Numeric)
	{
		return MakeValueType(MakeConcreteType(TypeDesc.ComponentType), TypeDesc.NumComponents);
	}
	return Type;
}

EValueType MakeDerivativeType(EValueType Type)
{
	const FValueTypeDescription& TypeDesc = GetValueTypeDescription(Type);
	if (IsNumericType(TypeDesc.ComponentType))
	{
		return MakeValueType(EValueComponentType::Float, TypeDesc.NumComponents);
	}
	return EValueType::Void;
}

FStructTypeRegistry::FStructTypeRegistry(FMemStackBase& InAllocator)
	: Allocator(&InAllocator)
{
}

void FStructTypeRegistry::EmitDeclarationsCode(FStringBuilderBase& OutCode) const
{
	for (const auto& It : Types)
	{
		const FStructType* StructType = It.Value;
		// Don't need to emit declaration for external types
		if (!StructType->IsExternal())
		{
			OutCode.Appendf(TEXT("struct %s\n"), StructType->Name);
			OutCode.Append(TEXT("{\n"));
			for (const FStructField& Field : StructType->Fields)
			{
				OutCode.Appendf(TEXT("\t%s %s;\n"), Field.Type.GetName(), Field.Name);
			}
			OutCode.Append(TEXT("};\n"));

			for (const FStructField& Field : StructType->Fields)
			{
				OutCode.Appendf(TEXT("%s %s_Set%s(%s Self, %s Value) { Self.%s = Value; return Self; }\n"),
					StructType->Name, StructType->Name, Field.Name, StructType->Name, Field.Type.GetName(), Field.Name);
			}
			OutCode.Append(TEXT("\n"));
		}
	}
}

namespace Private
{
void SetFieldType(EValueType* FieldTypes, EValueComponentType* ComponentTypes, int32 FieldIndex, int32 ComponentIndex, const FType& Type)
{
	if (Type.IsStruct())
	{
		for (const FStructField& Field : Type.StructType->Fields)
		{
			SetFieldType(FieldTypes, ComponentTypes, FieldIndex + Field.FlatFieldIndex, ComponentIndex + Field.ComponentIndex, Field.Type);
		}
	}
	else
	{
		FieldTypes[FieldIndex] = Type.ValueType;
		const FValueTypeDescription& TypeDesc = GetValueTypeDescription(Type.ValueType);
		for (int32 i = 0; i < TypeDesc.NumComponents; ++i)
		{
			ComponentTypes[ComponentIndex + i] = TypeDesc.ComponentType;
		}
	}
}
} // namespace Private

const FStructType* FStructTypeRegistry::NewType(const FStructTypeInitializer& Initializer)
{
	TArray<FStructFieldInitializer, TInlineAllocator<16>> DerivativeFields;

	const int32 NumFields = Initializer.Fields.Num();
	FStructField* Fields = new(*Allocator) FStructField[NumFields];
	int32 ComponentIndex = 0;
	int32 FlatFieldIndex = 0;
	uint64 Hash = 0u;
	{
		FXxHash64Builder Hasher;
		Hasher.Update(Initializer.Name.GetData(), Initializer.Name.Len() * sizeof(TCHAR));

		for (int32 FieldIndex = 0; FieldIndex < NumFields; ++FieldIndex)
		{
			const FStructFieldInitializer& FieldInitializer = Initializer.Fields[FieldIndex];
			const FType& FieldType = FieldInitializer.Type;

			Hasher.Update(FieldInitializer.Name.GetData(), FieldInitializer.Name.Len() * sizeof(TCHAR));
			if (FieldType.IsStruct())
			{
				Hasher.Update(&FieldType.StructType->Hash, sizeof(FieldType.StructType->Hash));
			}
			else
			{
				Hasher.Update(&FieldType.ValueType, sizeof(FieldType.ValueType));
			}

			FStructField& Field = Fields[FieldIndex];
			Field.Name = MemStack::AllocateString(*Allocator, FieldInitializer.Name);
			Field.Type = FieldType;
			Field.ComponentIndex = ComponentIndex;
			Field.FlatFieldIndex = FlatFieldIndex;
			ComponentIndex += FieldType.GetNumComponents();
			FlatFieldIndex += FieldType.GetNumFlatFields();

			if (!Initializer.bIsDerivativeType)
			{
				const FType FieldDerivativeType = FieldType.GetDerivativeType();
				if (!FieldDerivativeType.IsVoid())
				{
					DerivativeFields.Emplace(FieldInitializer.Name, FieldDerivativeType);
				}
			}
		}
		Hash = Hasher.Finalize().Hash;
	}

	FStructType const* const* PrevStructType = Types.Find(Hash);
	if (PrevStructType)
	{
		return *PrevStructType;
	}

	EValueComponentType* ComponentTypes = new(*Allocator) EValueComponentType[ComponentIndex];
	EValueType* FlatFieldTypes = new(*Allocator) EValueType[FlatFieldIndex];
	for (int32 FieldIndex = 0; FieldIndex < NumFields; ++FieldIndex)
	{
		const FStructField& Field = Fields[FieldIndex];
		Private::SetFieldType(FlatFieldTypes, ComponentTypes, Field.FlatFieldIndex, Field.ComponentIndex, Field.Type);
	}

	FStructType* StructType = new(*Allocator) FStructType();
	StructType->Name = MemStack::AllocateString(*Allocator, Initializer.Name);
	StructType->Hash = Hash;
	StructType->Fields = MakeArrayView(Fields, NumFields);
	StructType->ComponentTypes = MakeArrayView(ComponentTypes, ComponentIndex);
	StructType->FlatFieldTypes = MakeArrayView(FlatFieldTypes, FlatFieldIndex);

	Types.Add(Hash, StructType);

	if (DerivativeFields.Num() > 0)
	{
		const FString DerivativeTypeName = FString(Initializer.Name) + TEXT("_Deriv");
		FStructTypeInitializer DerivativeTypeInitializer;
		DerivativeTypeInitializer.Name = DerivativeTypeName;
		DerivativeTypeInitializer.Fields = DerivativeFields;
		DerivativeTypeInitializer.bIsDerivativeType = true;
		StructType->DerivativeType = NewType(DerivativeTypeInitializer);
	}

	return StructType;
}

const FStructType* FStructTypeRegistry::NewExternalType(FStringView Name)
{
	uint64 Hash = 0u;
	{
		FXxHash64Builder Hasher;
		Hasher.Update(Name.GetData(), Name.Len() * sizeof(TCHAR));
		Hash = Hasher.Finalize().Hash;
	}

	FStructType* StructType = new(*Allocator) FStructType();
	StructType->Name = MemStack::AllocateString(*Allocator, Name);
	StructType->Hash = Hash;
	Types.Add(Hash, StructType);
	return StructType;
}

const FStructType* FStructTypeRegistry::FindType(uint64 Hash) const
{
	FStructType const* const* PrevType = Types.Find(Hash);
	return PrevType ? *PrevType : nullptr;
}

namespace Private
{

/** Converts an arbitrary number into a safe divisor. i.e. FMath::Abs(Number) >= DELTA */
/**
 * FORCENOINLINE is required to discourage compiler from vectorizing the Div operation, which may tempt it into optimizing divide as A * rcp(B)
 * This will break shaders that are depending on exact divide results (see SubUV material function)
 * Technically this could still happen for a scalar divide, but it doesn't seem to occur in practice
 */
template<typename T>
FORCENOINLINE T GetSafeDivisor(T Number)
{
	if (FMath::Abs(Number) < (T)UE_DELTA)
	{
		if (Number < 0)
		{
			return -(T)UE_DELTA;
		}
		else
		{
			return (T)UE_DELTA;
		}
	}
	else
	{
		return Number;
	}
}

template<>
FORCENOINLINE int32 GetSafeDivisor<int32>(int32 Number)
{
	return Number != 0 ? Number : 1;
}

struct FOpBase
{
	static constexpr bool SupportsDouble = true;
	static constexpr bool SupportsInt = true;
};

struct FOpBaseNoInt : public FOpBase
{
	static constexpr bool SupportsInt = false;
};

struct FOpNeg : public FOpBase { template<typename T> T operator()(T Value) const { return -Value; } };
struct FOpAbs : public FOpBase { template<typename T> T operator()(T Value) const { return FMath::Abs(Value); } };
struct FOpSign : public FOpBase { template<typename T> T operator()(T Value) const { return FMath::Sign(Value); } };
struct FOpSaturate : public FOpBaseNoInt { template<typename T> T operator()(T Value) const { return FMath::Clamp(Value, (T)0, (T)1); } };
struct FOpFloor : public FOpBaseNoInt { template<typename T> T operator()(T Value) const { return FMath::FloorToFloat(Value); } };
struct FOpCeil : public FOpBaseNoInt { template<typename T> T operator()(T Value) const { return FMath::CeilToFloat(Value); } };
struct FOpRound : public FOpBaseNoInt { template<typename T> T operator()(T Value) const { return FMath::RoundToFloat(Value); } };
struct FOpTrunc : public FOpBaseNoInt { template<typename T> T operator()(T Value) const { return FMath::TruncToFloat(Value); } };
struct FOpFrac : public FOpBaseNoInt { template<typename T> T operator()(T Value) const { return FMath::Frac(Value); } };
struct FOpFractional : public FOpBaseNoInt { template<typename T> T operator()(T Value) const { return FMath::Fractional(Value); } };
struct FOpSqrt : public FOpBaseNoInt { template<typename T> T operator()(T Value) const { return FMath::Sqrt(Value); } };
struct FOpRcp : public FOpBaseNoInt { template<typename T> T operator()(T Value) const { return (T)1 / GetSafeDivisor(Value); } };
struct FOpExp : public FOpBaseNoInt { template<typename T> T operator()(T Value) const { return FMath::Exp(Value); } };
struct FOpExp2 : public FOpBaseNoInt { template<typename T> T operator()(T Value) const { return FMath::Exp2(Value); } };
struct FOpLog : public FOpBaseNoInt { template<typename T> T operator()(T Value) const { return FMath::Loge(Value); } };
struct FOpLog2 : public FOpBaseNoInt { template<typename T> T operator()(T Value) const { return FMath::Log2(Value); } };
struct FOpLog10 : public FOpBaseNoInt
{
	template<typename T> T operator()(T Value) const
	{
		static const T LogToLog10 = (T)1.0 / FMath::Loge((T)10);
		return FMath::Loge(Value) * LogToLog10;
	}
};
struct FOpSin : public FOpBaseNoInt { template<typename T> T operator()(T Value) const { return FMath::Sin(Value); } };
struct FOpCos : public FOpBaseNoInt { template<typename T> T operator()(T Value) const { return FMath::Cos(Value); } };
struct FOpTan : public FOpBaseNoInt { template<typename T> T operator()(T Value) const { return FMath::Tan(Value); } };
struct FOpAsin : public FOpBaseNoInt { template<typename T> T operator()(T Value) const { return FMath::Asin(Value); } };
struct FOpAcos : public FOpBaseNoInt { template<typename T> T operator()(T Value) const { return FMath::Acos(Value); } };
struct FOpAtan : public FOpBaseNoInt { template<typename T> T operator()(T Value) const { return FMath::Atan(Value); } };
struct FOpAdd : public FOpBase { template<typename T> T operator()(T Lhs, T Rhs) const { return Lhs + Rhs; } };
struct FOpSub : public FOpBase { template<typename T> T operator()(T Lhs, T Rhs) const { return Lhs - Rhs; } };
struct FOpMul : public FOpBase { template<typename T> T operator()(T Lhs, T Rhs) const { return Lhs * Rhs; } };
struct FOpDiv : public FOpBase { template<typename T> T operator()(T Lhs, T Rhs) const { return Lhs / GetSafeDivisor(Rhs); } };
struct FOpMin : public FOpBase { template<typename T> T operator()(T Lhs, T Rhs) const { return FMath::Min(Lhs, Rhs); } };
struct FOpMax : public FOpBase { template<typename T> T operator()(T Lhs, T Rhs) const { return FMath::Max(Lhs, Rhs); } };
struct FOpFmod : public FOpBaseNoInt { template<typename T> T operator()(T Lhs, T Rhs) const { return FMath::Fmod(Lhs, GetSafeDivisor(Rhs)); } };
struct FOpModulo : public FOpBase
{
	static float  ModuloImpl(float Lhs, float Rhs)   { return FMath::Fmod(Lhs, Rhs); }
	static double ModuloImpl(double Lhs, double Rhs) { return FMath::Fmod(Lhs, Rhs); }
	static int32  ModuloImpl(int32 Lhs, int32 Rhs)   { return Lhs % Rhs;             }

	template<typename T> T operator()(T Lhs, T Rhs) const
	{
		return ModuloImpl(Lhs, GetSafeDivisor(Rhs));
	}
};
struct FOpAtan2 : public FOpBaseNoInt { template<typename T> T operator()(T Lhs, T Rhs) const { return FMath::Atan2(Lhs, Rhs); } };
struct FOpLess : public FOpBase { template<typename T> bool operator()(T Lhs, T Rhs) const { return Lhs < Rhs; } };
struct FOpGreater : public FOpBase { template<typename T> bool operator()(T Lhs, T Rhs) const { return Lhs > Rhs; } };
struct FOpLessEqual : public FOpBase { template<typename T> bool operator()(T Lhs, T Rhs) const { return Lhs <= Rhs; } };
struct FOpGreaterEqual : public FOpBase { template<typename T> bool operator()(T Lhs, T Rhs) const { return Lhs >= Rhs; } };

template<typename Operation>
inline FValue UnaryOp(const Operation& Op, const FValue& Value)
{
	if (Value.Type.IsStruct())
	{
		return FValue();
	}
	const FValueTypeDescription& TypeDesc = GetValueTypeDescription(Value.Type);
	check(TypeDesc.ComponentType != EValueComponentType::Numeric);
	const int8 NumComponents = TypeDesc.NumComponents;
	
	FValue Result;
	if constexpr (Operation::SupportsDouble)
	{
		if (TypeDesc.ComponentType == EValueComponentType::Double)
		{
			Result.Type = MakeValueType(EValueComponentType::Double, NumComponents);
			const FDoubleValue Cast = Value.AsDouble();
			for (int32 i = 0; i < NumComponents; ++i)
			{
				Result.Component.Add(Op(Cast.Component[i]));
			}
			return Result;
		}
	}

	if constexpr (Operation::SupportsInt)
	{
		if (TypeDesc.ComponentType != EValueComponentType::Float)
		{
			Result.Type = MakeValueType(EValueComponentType::Int, NumComponents);
			const FIntValue Cast = Value.AsInt();
			for (int32 i = 0; i < NumComponents; ++i)
			{
				Result.Component.Add(Op(Cast.Component[i]));
			}
			return Result;
		}
	}

	Result.Type = MakeValueType(EValueComponentType::Float, NumComponents);
	const FFloatValue Cast = Value.AsFloat();
	for (int32 i = 0; i < NumComponents; ++i)
	{
		Result.Component.Add(Op(Cast.Component[i]));
	}
	return Result;
}

// Similar to "UnaryOp", but reads and writes results to a loose FValueComponent array with a specified initial type.
// Certain ops may change the type, so a new type is returned, but it will always have the same number of components.
template<typename Operation>
inline EValueType UnaryOpInPlace(const Operation& Op, EValueType Type, TArrayView<FValueComponent>& Component)
{
	const FValueTypeDescription& TypeDesc = GetValueTypeDescription(Type);
	check(TypeDesc.ComponentType != EValueComponentType::Numeric);
	const int8 NumComponents = TypeDesc.NumComponents;

	EValueType ResultType;

	// Check for Float input first, as it's by far the most common case (96%)
	if (TypeDesc.ComponentType == EValueComponentType::Float)
	{
		// Float to float, no need to recompute type or cast
		ResultType = Type;
		for (int32 i = 0; i < NumComponents; ++i)
		{
			Component[i].Float = Op(Component[i].Float);
		}
		return ResultType;
	}

	if constexpr (Operation::SupportsDouble)
	{
		if (TypeDesc.ComponentType == EValueComponentType::Double)
		{
			// Double to double, no need to recompute type or cast
			ResultType = Type;
			for (int32 i = 0; i < NumComponents; ++i)
			{
				Component[i].Double = Op(Component[i].Double);
			}
			return ResultType;
		}
	}

	if constexpr (Operation::SupportsInt)
	{
		// Cast to integer
		ResultType = MakeValueType(EValueComponentType::Int, NumComponents);
		const FCastInt Cast;
		for (int32 i = 0; i < NumComponents; ++i)
		{
			Component[i].Int = Op(Cast(TypeDesc.ComponentType,Component[i]));
		}
		return ResultType;
	}
	else
	{
		// Cast to float
		ResultType = MakeValueType(EValueComponentType::Float, NumComponents);
		const FCastFloat Cast;
		for (int32 i = 0; i < NumComponents; ++i)
		{
			Component[i].Float = Op(Cast(TypeDesc.ComponentType, Component[i]));
		}
		return ResultType;
	}
}

inline int8 GetNumComponentsResult(int8 Lhs, int8 Rhs)
{
	// operations between scalar and non-scalar will splat the scalar value
	// otherwise, operations should only be between types with same number of components
	return (Lhs == 1 || Rhs == 1) ? FMath::Max(Lhs, Rhs) : FMath::Min(Lhs, Rhs);
}

template<typename Operation>
inline FValue BinaryOp(const Operation& Op, const FValue& Lhs, const FValue& Rhs)
{
	if (Lhs.Type.IsStruct() || Rhs.Type.IsStruct())
	{
		return FValue();
	}
	const FValueTypeDescription& LhsDesc = GetValueTypeDescription(Lhs.Type);
	const FValueTypeDescription& RhsDesc = GetValueTypeDescription(Rhs.Type);
	check(LhsDesc.ComponentType != EValueComponentType::Numeric);
	check(RhsDesc.ComponentType != EValueComponentType::Numeric);
	const int8 NumComponents = GetNumComponentsResult(LhsDesc.NumComponents, RhsDesc.NumComponents);
	
	FValue Result;
	if constexpr (Operation::SupportsDouble)
	{
		if (LhsDesc.ComponentType == EValueComponentType::Double || RhsDesc.ComponentType == EValueComponentType::Double)
		{
			Result.Type = MakeValueType(EValueComponentType::Double, NumComponents);
			const FDoubleValue LhsCast = Lhs.AsDouble();
			const FDoubleValue RhsCast = Rhs.AsDouble();
			for (int32 i = 0; i < NumComponents; ++i)
			{
				Result.Component.Add(Op(LhsCast.Component[i], RhsCast.Component[i]));
			}
			return Result;
		}
	}

	if constexpr (Operation::SupportsInt)
	{
		if (LhsDesc.ComponentType != EValueComponentType::Float && RhsDesc.ComponentType != EValueComponentType::Float)
		{
			Result.Type = MakeValueType(EValueComponentType::Int, NumComponents);
			const FIntValue LhsCast = Lhs.AsInt();
			const FIntValue RhsCast = Rhs.AsInt();
			for (int32 i = 0; i < NumComponents; ++i)
			{
				Result.Component.Add(Op(LhsCast.Component[i], RhsCast.Component[i]));
			}
			return Result;
		}
	}

	Result.Type = MakeValueType(EValueComponentType::Float, NumComponents);
	const FFloatValue LhsCast = Lhs.AsFloat();
	const FFloatValue RhsCast = Rhs.AsFloat();
	for (int32 i = 0; i < NumComponents; ++i)
	{
		Result.Component.Add(Op(LhsCast.Component[i], RhsCast.Component[i]));
	}
	return Result;
}

// Similar to "BinaryOp", but reads and writes results to a loose FValueComponent array with specified types for Lhs and Rhs.
// The components are assumed to be sequential in the array, with Lhs first and Rhs second.  The result overwrites the start
// of the Component array, and may have a different (but always smaller) number of components and type from the original values.
// The new type is returned, and the number of components consumed by the operation is stored in OutComponentsConsumed.  The
// consumed number can be removed from the Component array by the caller to produce a final result array.
template<typename Operation>
inline EValueType BinaryOpInPlace(const Operation& Op, EValueType LhsType, EValueType RhsType, TArrayView<FValueComponent>& Component, int32& OutComponentsConsumed)
{
	const FValueTypeDescription& LhsDesc = GetValueTypeDescription(LhsType);
	const FValueTypeDescription& RhsDesc = GetValueTypeDescription(RhsType);
	check(LhsDesc.ComponentType != EValueComponentType::Numeric);
	check(RhsDesc.ComponentType != EValueComponentType::Numeric);
	const int8 NumComponents = GetNumComponentsResult(LhsDesc.NumComponents, RhsDesc.NumComponents);

	OutComponentsConsumed = LhsDesc.NumComponents + RhsDesc.NumComponents - NumComponents;
	check(OutComponentsConsumed >= 0);

	EValueType ResultType;

	// Fast path for both input and output float (common case)
	if (LhsDesc.ComponentType == EValueComponentType::Float && RhsDesc.ComponentType == EValueComponentType::Float)
	{
		ResultType = MakeValueType(EValueComponentType::Float, NumComponents);

		// Handle splatting of scalars, by choosing a zero or one increment
		int32 LhsIncrement = (LhsDesc.NumComponents == 1) ? 0 : 1;
		int32 RhsIncrement = (RhsDesc.NumComponents == 1) ? 0 : 1;
		int32 LhsComponent = 0;
		int32 RhsComponent = LhsDesc.NumComponents;

		// Need to write to temporary, since output can overlap with input
		FFloatValue TempResult;
		for (int32 i = 0; i < NumComponents; i++, LhsComponent += LhsIncrement, RhsComponent += RhsIncrement)
		{
			TempResult.Component[i] = Op(Component[LhsComponent].Float, Component[RhsComponent].Float);
		}
		for (int32 i = 0; i < NumComponents; i++)
		{
			Component[i].Float = TempResult.Component[i];
		}

		return ResultType;
	}

	if constexpr (Operation::SupportsDouble)
	{
		if (LhsDesc.ComponentType == EValueComponentType::Double || RhsDesc.ComponentType == EValueComponentType::Double)
		{
			ResultType = MakeValueType(EValueComponentType::Double, NumComponents);

			FDoubleValue LhsCast;
			FDoubleValue RhsCast;
			Private::AsTypeInPlace(Private::FCastDouble(), LhsType, TArrayView<FValueComponent>(Component.GetData(),  LhsDesc.NumComponents), LhsCast);
			Private::AsTypeInPlace(Private::FCastDouble(), RhsType, TArrayView<FValueComponent>(Component.GetData() + LhsDesc.NumComponents, RhsDesc.NumComponents), RhsCast);

			for (int32 i = 0; i < NumComponents; ++i)
			{
				Component[i].Double = Op(LhsCast.Component[i], RhsCast.Component[i]);
			}
			return ResultType;
		}
	}

	if constexpr (Operation::SupportsInt)
	{
		if (LhsDesc.ComponentType != EValueComponentType::Float && RhsDesc.ComponentType != EValueComponentType::Float)
		{
			ResultType = MakeValueType(EValueComponentType::Int, NumComponents);

			FIntValue LhsCast;
			FIntValue RhsCast;
			Private::AsTypeInPlace(Private::FCastInt(), LhsType, TArrayView<FValueComponent>(Component.GetData(),  LhsDesc.NumComponents), LhsCast);
			Private::AsTypeInPlace(Private::FCastInt(), RhsType, TArrayView<FValueComponent>(Component.GetData() + LhsDesc.NumComponents, RhsDesc.NumComponents), RhsCast);

			for (int32 i = 0; i < NumComponents; ++i)
			{
				Component[i].Int = Op(LhsCast.Component[i], RhsCast.Component[i]);
			}
			return ResultType;
		}
	}

	ResultType = MakeValueType(EValueComponentType::Float, NumComponents);

	FFloatValue LhsCast;
	FFloatValue RhsCast;
	Private::AsTypeInPlace(Private::FCastFloat(), LhsType, TArrayView<FValueComponent>(Component.GetData(),  LhsDesc.NumComponents), LhsCast);
	Private::AsTypeInPlace(Private::FCastFloat(), RhsType, TArrayView<FValueComponent>(Component.GetData() + LhsDesc.NumComponents, RhsDesc.NumComponents), RhsCast);

	for (int32 i = 0; i < NumComponents; ++i)
	{
		Component[i].Float = Op(LhsCast.Component[i], RhsCast.Component[i]);
	}
	return ResultType;
}

template<typename Operation>
inline FValue CompareOp(const Operation& Op, const FValue& Lhs, const FValue& Rhs)
{
	if (Lhs.Type.IsStruct() || Rhs.Type.IsStruct())
	{
		return FValue();
	}
	const FValueTypeDescription& LhsDesc = GetValueTypeDescription(Lhs.Type);
	const FValueTypeDescription& RhsDesc = GetValueTypeDescription(Rhs.Type);
	check(LhsDesc.ComponentType != EValueComponentType::Numeric);
	check(RhsDesc.ComponentType != EValueComponentType::Numeric);
	const int8 NumComponents = GetNumComponentsResult(LhsDesc.NumComponents, RhsDesc.NumComponents);

	FValue Result;
	Result.Type = MakeValueType(EValueComponentType::Bool, NumComponents);
	if constexpr (Operation::SupportsDouble)
	{
		if (LhsDesc.ComponentType == EValueComponentType::Double || RhsDesc.ComponentType == EValueComponentType::Double)
		{
			const FDoubleValue LhsCast = Lhs.AsDouble();
			const FDoubleValue RhsCast = Rhs.AsDouble();
			for (int32 i = 0; i < NumComponents; ++i)
			{
				Result.Component.Add(Op(LhsCast.Component[i], RhsCast.Component[i]));
			}
			return Result;
		}
	}

	if constexpr (Operation::SupportsInt)
	{
		if (LhsDesc.ComponentType != EValueComponentType::Float && RhsDesc.ComponentType != EValueComponentType::Float)
		{
			const FIntValue LhsCast = Lhs.AsInt();
			const FIntValue RhsCast = Rhs.AsInt();
			for (int32 i = 0; i < NumComponents; ++i)
			{
				Result.Component.Add(Op(LhsCast.Component[i], RhsCast.Component[i]));
			}
			return Result;
		}
	}

	const FFloatValue LhsCast = Lhs.AsFloat();
	const FFloatValue RhsCast = Rhs.AsFloat();
	for (int32 i = 0; i < NumComponents; ++i)
	{
		Result.Component.Add(Op(LhsCast.Component[i], RhsCast.Component[i]));
	}
	return Result;
}

} // namespace Private

bool operator==(const FValue& Lhs, const FValue& Rhs)
{
	if (Lhs.Type != Rhs.Type)
	{
		return false;
	}

	check(Lhs.Component.Num() == Rhs.Component.Num());
	for (int32 i = 0u; i < Lhs.Component.Num(); ++i)
	{
		if (Lhs.Component[i].Packed != Rhs.Component[i].Packed)
		{
			return false;
		}
	}
	return true;
}

uint32 GetTypeHash(const FType& Type)
{
	uint32 Result = ::GetTypeHash(Type.ValueType);
	if (Type.IsStruct())
	{
		Result = HashCombine(Result, ::GetTypeHash(Type.StructType));
	}
	return Result;
}

uint32 GetTypeHash(const FValue& Value)
{
	uint32 Result = GetTypeHash(Value.Type);
	const int32 NumComponents = Value.Type.GetNumComponents();
	for(int32 Index = 0; Index < NumComponents; ++Index)
	{
		uint32 ComponentHash = 0u;
		switch (Value.Type.GetComponentType(Index))
		{
		case EValueComponentType::Float: ComponentHash = ::GetTypeHash(Value.Component[Index].Float); break;
		case EValueComponentType::Double: ComponentHash = ::GetTypeHash(Value.Component[Index].Double); break;
		case EValueComponentType::Int: ComponentHash = ::GetTypeHash(Value.Component[Index].Int); break;
		case EValueComponentType::Bool: ComponentHash = ::GetTypeHash(Value.Component[Index].Bool); break;
		case EValueComponentType::Numeric: ComponentHash = ::GetTypeHash(Value.Component[Index].Double); break;
		default: checkNoEntry(); break;
		}
		Result = HashCombine(Result, ComponentHash);
	}
	return Result;
}

FValue Neg(const FValue& Value)
{
	return Private::UnaryOp(Private::FOpNeg(), Value);
}

FValue Abs(const FValue& Value)
{
	return Private::UnaryOp(Private::FOpAbs(), Value);
}

FValue Saturate(const FValue& Value)
{
	return Private::UnaryOp(Private::FOpSaturate(), Value);
}

FValue Floor(const FValue& Value)
{
	return Private::UnaryOp(Private::FOpFloor(), Value);
}

FValue Ceil(const FValue& Value)
{
	return Private::UnaryOp(Private::FOpCeil(), Value);
}

FValue Round(const FValue& Value)
{
	return Private::UnaryOp(Private::FOpRound(), Value);
}

FValue Trunc(const FValue& Value)
{
	return Private::UnaryOp(Private::FOpTrunc(), Value);
}

FValue Sign(const FValue& Value)
{
	return Private::UnaryOp(Private::FOpSign(), Value);
}

FValue Frac(const FValue& Value)
{
	return Private::UnaryOp(Private::FOpFrac(), Value);
}

FValue Fractional(const FValue& Value)
{
	return Private::UnaryOp(Private::FOpFractional(), Value);
}

FValue Sqrt(const FValue& Value)
{
	return Private::UnaryOp(Private::FOpSqrt(), Value);
}

FValue Rcp(const FValue& Value)
{
	return Private::UnaryOp(Private::FOpRcp(), Value);
}

FValue Exp(const FValue& Value)
{
	return Private::UnaryOp(Private::FOpExp(), Value);
}

FValue Exp2(const FValue& Value)
{
	return Private::UnaryOp(Private::FOpExp2(), Value);
}

FValue Log(const FValue& Value)
{
	return Private::UnaryOp(Private::FOpLog(), Value);
}

FValue Log2(const FValue& Value)
{
	return Private::UnaryOp(Private::FOpLog2(), Value);
}

FValue Log10(const FValue& Value)
{
	return Private::UnaryOp(Private::FOpLog10(), Value);
}

FValue Sin(const FValue& Value)
{
	return Private::UnaryOp(Private::FOpSin(), Value);
}

FValue Cos(const FValue& Value)
{
	return Private::UnaryOp(Private::FOpCos(), Value);
}

FValue Tan(const FValue& Value)
{
	return Private::UnaryOp(Private::FOpTan(), Value);
}

FValue Asin(const FValue& Value)
{
	return Private::UnaryOp(Private::FOpAsin(), Value);
}

FValue Acos(const FValue& Value)
{
	return Private::UnaryOp(Private::FOpAcos(), Value);
}

FValue Atan(const FValue& Value)
{
	return Private::UnaryOp(Private::FOpAtan(), Value);
}

FValue Add(const FValue& Lhs, const FValue& Rhs)
{
	return Private::BinaryOp(Private::FOpAdd(), Lhs, Rhs);
}

FValue Sub(const FValue& Lhs, const FValue& Rhs)
{
	return Private::BinaryOp(Private::FOpSub(), Lhs, Rhs);
}

FValue Mul(const FValue& Lhs, const FValue& Rhs)
{
	return Private::BinaryOp(Private::FOpMul(), Lhs, Rhs);
}

FValue Div(const FValue& Lhs, const FValue& Rhs)
{
	return Private::BinaryOp(Private::FOpDiv(), Lhs, Rhs);
}

FValue Less(const FValue& Lhs, const FValue& Rhs)
{
	return Private::CompareOp(Private::FOpLess(), Lhs, Rhs);
}

FValue Greater(const FValue& Lhs, const FValue& Rhs)
{
	return Private::CompareOp(Private::FOpGreater(), Lhs, Rhs);
}

FValue LessEqual(const FValue& Lhs, const FValue& Rhs)
{
	return Private::CompareOp(Private::FOpLessEqual(), Lhs, Rhs);
}

FValue GreaterEqual(const FValue& Lhs, const FValue& Rhs)
{
	return Private::CompareOp(Private::FOpGreaterEqual(), Lhs, Rhs);
}

FValue Min(const FValue& Lhs, const FValue& Rhs)
{
	return Private::BinaryOp(Private::FOpMin(), Lhs, Rhs);
}

FValue Max(const FValue& Lhs, const FValue& Rhs)
{
	return Private::BinaryOp(Private::FOpMax(), Lhs, Rhs);
}

FValue Fmod(const FValue& Lhs, const FValue& Rhs)
{
	return Private::BinaryOp(Private::FOpFmod(), Lhs, Rhs);
}

FValue Modulo(const FValue& Lhs, const FValue& Rhs)
{
	return Private::BinaryOp(Private::FOpModulo(), Lhs, Rhs);
}

FValue Atan2(const FValue& Lhs, const FValue& Rhs)
{
	return Private::BinaryOp(Private::FOpAtan2(), Lhs, Rhs);
}

FValue Clamp(const FValue& Value, const FValue& Low, const FValue& High)
{
	return Min(Max(Value, Low), High);
}

FValue Dot(const FValue& Lhs, const FValue& Rhs)
{
	if (Lhs.Type.IsStruct() || Rhs.Type.IsStruct())
	{
		return FValue();
	}
	const FValueTypeDescription& LhsDesc = GetValueTypeDescription(Lhs.Type);
	const FValueTypeDescription& RhsDesc = GetValueTypeDescription(Rhs.Type);
	const int8 NumComponents = Private::GetNumComponentsResult(LhsDesc.NumComponents, RhsDesc.NumComponents);

	FValue Result;
	if (LhsDesc.ComponentType == EValueComponentType::Double || RhsDesc.ComponentType == EValueComponentType::Double)
	{
		Result.Type = EValueType::Double1;
		const FDoubleValue LhsValue = Lhs.AsDouble();
		const FDoubleValue RhsValue = Rhs.AsDouble();
		double ComponentValue = 0.0;
		for (int32 i = 0; i < NumComponents; ++i)
		{
			ComponentValue += LhsValue.Component[i] * RhsValue.Component[i];
		}
		Result.Component.Add(ComponentValue);
	}
	else if (LhsDesc.ComponentType == EValueComponentType::Float || RhsDesc.ComponentType == EValueComponentType::Float)
	{
		Result.Type = EValueType::Float1;
		const FFloatValue LhsValue = Lhs.AsFloat();
		const FFloatValue RhsValue = Rhs.AsFloat();
		float ComponentValue = 0.0f;
		for (int32 i = 0; i < NumComponents; ++i)
		{
			ComponentValue += LhsValue.Component[i] * RhsValue.Component[i];
		}
		Result.Component.Add(ComponentValue);
	}
	else
	{
		Result.Type = EValueType::Int1;
		const FIntValue LhsValue = Lhs.AsInt();
		const FIntValue RhsValue = Rhs.AsInt();
		int32 ComponentValue = 0;
		for (int32 i = 0; i < NumComponents; ++i)
		{
			ComponentValue += LhsValue.Component[i] * RhsValue.Component[i];
		}
		Result.Component.Add(ComponentValue);
	}
	return Result;
}

FValue Cross(const FValue& Lhs, const FValue& Rhs)
{
	if (Lhs.Type.IsStruct() || Rhs.Type.IsStruct())
	{
		return FValue();
	}
	const FValueTypeDescription& LhsDesc = GetValueTypeDescription(Lhs.Type);
	const FValueTypeDescription& RhsDesc = GetValueTypeDescription(Rhs.Type);

	FValue Result;
	if (LhsDesc.ComponentType == EValueComponentType::Double || RhsDesc.ComponentType == EValueComponentType::Double)
	{
		Result.Type = EValueType::Double3;
		const FDoubleValue LhsValue = Lhs.AsDouble();
		const FDoubleValue RhsValue = Rhs.AsDouble();

		Result.Component.Add(LhsValue.Component[1] * RhsValue.Component[2] - LhsValue.Component[2] * RhsValue.Component[1]);
		Result.Component.Add(LhsValue.Component[2] * RhsValue.Component[0] - LhsValue.Component[0] * RhsValue.Component[2]);
		Result.Component.Add(LhsValue.Component[0] * RhsValue.Component[1] - LhsValue.Component[1] * RhsValue.Component[0]);
	}
	else if (LhsDesc.ComponentType == EValueComponentType::Float || RhsDesc.ComponentType == EValueComponentType::Float)
	{
		Result.Type = EValueType::Float3;
		const FFloatValue LhsValue = Lhs.AsFloat();
		const FFloatValue RhsValue = Rhs.AsFloat();
		
		Result.Component.Add(LhsValue.Component[1] * RhsValue.Component[2] - LhsValue.Component[2] * RhsValue.Component[1]);
		Result.Component.Add(LhsValue.Component[2] * RhsValue.Component[0] - LhsValue.Component[0] * RhsValue.Component[2]);
		Result.Component.Add(LhsValue.Component[0] * RhsValue.Component[1] - LhsValue.Component[1] * RhsValue.Component[0]);
	}
	else
	{
		Result.Type = EValueType::Int3;
		const FIntValue LhsValue = Lhs.AsInt();
		const FIntValue RhsValue = Rhs.AsInt();

		Result.Component.Add(LhsValue.Component[1] * RhsValue.Component[2] - LhsValue.Component[2] * RhsValue.Component[1]);
		Result.Component.Add(LhsValue.Component[2] * RhsValue.Component[0] - LhsValue.Component[0] * RhsValue.Component[2]);
		Result.Component.Add(LhsValue.Component[0] * RhsValue.Component[1] - LhsValue.Component[1] * RhsValue.Component[0]);
	}
	return Result;
}

FValue Append(const FValue& Lhs, const FValue& Rhs)
{
	if (Lhs.Type.IsStruct() || Rhs.Type.IsStruct())
	{
		return FValue();
	}
	const FValueTypeDescription& LhsDesc = GetValueTypeDescription(Lhs.Type);
	const FValueTypeDescription& RhsDesc = GetValueTypeDescription(Rhs.Type);

	FValue Result;
	const int32 NumComponents = FMath::Min<int32>(LhsDesc.NumComponents + RhsDesc.NumComponents, 4);
	if (LhsDesc.ComponentType == RhsDesc.ComponentType)
	{
		// If both values have the same component type, use as-is
		// (otherwise will need to convert)
		
		Result.Type = MakeValueType(LhsDesc.ComponentType, NumComponents);
		for (int32 i = 0; i < LhsDesc.NumComponents && Result.Component.Num() < NumComponents; ++i)
		{
			Result.Component.Add(Lhs.Component[i]);
		}
		for (int32 i = 0; i < RhsDesc.NumComponents && Result.Component.Num() < NumComponents; ++i)
		{
			Result.Component.Add(Rhs.Component[i]);
		}
	}
	else if (LhsDesc.ComponentType == EValueComponentType::Double || RhsDesc.ComponentType == EValueComponentType::Double)
	{
		Result.Type = MakeValueType(EValueComponentType::Double, NumComponents);
		const FDoubleValue LhsValue = Lhs.AsDouble();
		const FDoubleValue RhsValue = Rhs.AsDouble();
		for (int32 i = 0; i < LhsDesc.NumComponents && Result.Component.Num() < NumComponents; ++i)
		{
			Result.Component.Add(LhsValue.Component[i]);
		}
		for (int32 i = 0; i < RhsDesc.NumComponents && Result.Component.Num() < NumComponents; ++i)
		{
			Result.Component.Add(RhsValue.Component[i]);
		}
	}
	else if (LhsDesc.ComponentType == EValueComponentType::Float || RhsDesc.ComponentType == EValueComponentType::Float)
	{
		Result.Type = MakeValueType(EValueComponentType::Float, NumComponents);
		const FFloatValue LhsValue = Lhs.AsFloat();
		const FFloatValue RhsValue = Rhs.AsFloat();
		for (int32 i = 0; i < LhsDesc.NumComponents && Result.Component.Num() < NumComponents; ++i)
		{
			Result.Component.Add(LhsValue.Component[i]);
		}
		for (int32 i = 0; i < RhsDesc.NumComponents && Result.Component.Num() < NumComponents; ++i)
		{
			Result.Component.Add(RhsValue.Component[i]);
		}
	}
	else
	{
		Result.Type = MakeValueType(EValueComponentType::Int, NumComponents);
		const FIntValue LhsValue = Lhs.AsInt();
		const FIntValue RhsValue = Rhs.AsInt();
		for (int32 i = 0; i < LhsDesc.NumComponents && Result.Component.Num() < NumComponents; ++i)
		{
			Result.Component.Add(LhsValue.Component[i]);
		}
		for (int32 i = 0; i < RhsDesc.NumComponents && Result.Component.Num() < NumComponents; ++i)
		{
			Result.Component.Add(RhsValue.Component[i]);
		}
	}
	return Result;
}

FValue Cast(const FValue& Value, EValueType Type)
{
	const EValueType SourceType = Value.GetType();
	if (Type == SourceType)
	{
		return Value;
	}

	FValue Result;
	Result.Type = Type;

	switch (GetValueTypeDescription(Type).ComponentType)
	{
	case EValueComponentType::Float: Private::Cast(Private::FCastFloat(), Value, Result); break;
	case EValueComponentType::Double: Private::Cast(Private::FCastDouble(), Value, Result); break;
	case EValueComponentType::Int: Private::Cast(Private::FCastInt(), Value, Result); break;
	case EValueComponentType::Bool: Private::Cast(Private::FCastBool(), Value, Result); break;
	default: checkNoEntry(); break;
	}

	return Result;
}


EValueType NegInPlace(EValueType Type, TArrayView<FValueComponent> Component)
{
	return Private::UnaryOpInPlace(Private::FOpNeg(), Type, Component);
}

EValueType AbsInPlace(EValueType Type, TArrayView<FValueComponent> Component)
{
	return Private::UnaryOpInPlace(Private::FOpAbs(), Type, Component);
}

EValueType SaturateInPlace(EValueType Type, TArrayView<FValueComponent> Component)
{
	return Private::UnaryOpInPlace(Private::FOpSaturate(), Type, Component);
}

EValueType FloorInPlace(EValueType Type, TArrayView<FValueComponent> Component)
{
	return Private::UnaryOpInPlace(Private::FOpFloor(), Type, Component);
}

EValueType CeilInPlace(EValueType Type, TArrayView<FValueComponent> Component)
{
	return Private::UnaryOpInPlace(Private::FOpCeil(), Type, Component);
}

EValueType RoundInPlace(EValueType Type, TArrayView<FValueComponent> Component)
{
	return Private::UnaryOpInPlace(Private::FOpRound(), Type, Component);
}

EValueType TruncInPlace(EValueType Type, TArrayView<FValueComponent> Component)
{
	return Private::UnaryOpInPlace(Private::FOpTrunc(), Type, Component);
}

EValueType SignInPlace(EValueType Type, TArrayView<FValueComponent> Component)
{
	return Private::UnaryOpInPlace(Private::FOpSign(), Type, Component);
}

EValueType FracInPlace(EValueType Type, TArrayView<FValueComponent> Component)
{
	return Private::UnaryOpInPlace(Private::FOpFrac(), Type, Component);
}

EValueType FractionalInPlace(EValueType Type, TArrayView<FValueComponent> Component)
{
	return Private::UnaryOpInPlace(Private::FOpFractional(), Type, Component);
}

EValueType SqrtInPlace(EValueType Type, TArrayView<FValueComponent> Component)
{
	return Private::UnaryOpInPlace(Private::FOpSqrt(), Type, Component);
}

EValueType RcpInPlace(EValueType Type, TArrayView<FValueComponent> Component)
{
	return Private::UnaryOpInPlace(Private::FOpRcp(), Type, Component);
}

EValueType ExpInPlace(EValueType Type, TArrayView<FValueComponent> Component)
{
	return Private::UnaryOpInPlace(Private::FOpExp(), Type, Component);
}

EValueType Exp2InPlace(EValueType Type, TArrayView<FValueComponent> Component)
{
	return Private::UnaryOpInPlace(Private::FOpExp2(), Type, Component);
}

EValueType LogInPlace(EValueType Type, TArrayView<FValueComponent> Component)
{
	return Private::UnaryOpInPlace(Private::FOpLog(), Type, Component);
}

EValueType Log2InPlace(EValueType Type, TArrayView<FValueComponent> Component)
{
	return Private::UnaryOpInPlace(Private::FOpLog2(), Type, Component);
}

EValueType Log10InPlace(EValueType Type, TArrayView<FValueComponent> Component)
{
	return Private::UnaryOpInPlace(Private::FOpLog10(), Type, Component);
}

EValueType SinInPlace(EValueType Type, TArrayView<FValueComponent> Component)
{
	return Private::UnaryOpInPlace(Private::FOpSin(), Type, Component);
}

EValueType CosInPlace(EValueType Type, TArrayView<FValueComponent> Component)
{
	return Private::UnaryOpInPlace(Private::FOpCos(), Type, Component);
}

EValueType TanInPlace(EValueType Type, TArrayView<FValueComponent> Component)
{
	return Private::UnaryOpInPlace(Private::FOpTan(), Type, Component);
}

EValueType AsinInPlace(EValueType Type, TArrayView<FValueComponent> Component)
{
	return Private::UnaryOpInPlace(Private::FOpAsin(), Type, Component);
}

EValueType AcosInPlace(EValueType Type, TArrayView<FValueComponent> Component)
{
	return Private::UnaryOpInPlace(Private::FOpAcos(), Type, Component);
}

EValueType AtanInPlace(EValueType Type, TArrayView<FValueComponent> Component)
{
	return Private::UnaryOpInPlace(Private::FOpAtan(), Type, Component);
}

EValueType AddInPlace(EValueType LhsType, EValueType RhsType, TArrayView<FValueComponent> Component, int32& OutComponentsConsumed)
{
	return Private::BinaryOpInPlace(Private::FOpAdd(), LhsType, RhsType, Component, OutComponentsConsumed);
}

EValueType SubInPlace(EValueType LhsType, EValueType RhsType, TArrayView<FValueComponent> Component, int32& OutComponentsConsumed)
{
	return Private::BinaryOpInPlace(Private::FOpSub(), LhsType, RhsType, Component, OutComponentsConsumed);
}

EValueType MulInPlace(EValueType LhsType, EValueType RhsType, TArrayView<FValueComponent> Component, int32& OutComponentsConsumed)
{
	return Private::BinaryOpInPlace(Private::FOpMul(), LhsType, RhsType, Component, OutComponentsConsumed);
}

EValueType DivInPlace(EValueType LhsType, EValueType RhsType, TArrayView<FValueComponent> Component, int32& OutComponentsConsumed)
{
	return Private::BinaryOpInPlace(Private::FOpDiv(), LhsType, RhsType, Component, OutComponentsConsumed);
}

EValueType MinInPlace(EValueType LhsType, EValueType RhsType, TArrayView<FValueComponent> Component, int32& OutComponentsConsumed)
{
	return Private::BinaryOpInPlace(Private::FOpMin(), LhsType, RhsType, Component, OutComponentsConsumed);
}

EValueType MaxInPlace(EValueType LhsType, EValueType RhsType, TArrayView<FValueComponent> Component, int32& OutComponentsConsumed)
{
	return Private::BinaryOpInPlace(Private::FOpMax(), LhsType, RhsType, Component, OutComponentsConsumed);
}

EValueType FmodInPlace(EValueType LhsType, EValueType RhsType, TArrayView<FValueComponent> Component, int32& OutComponentsConsumed)
{
	return Private::BinaryOpInPlace(Private::FOpFmod(), LhsType, RhsType, Component, OutComponentsConsumed);
}

EValueType ModuloInPlace(EValueType LhsType, EValueType RhsType, TArrayView<FValueComponent> Component, int32& OutComponentsConsumed)
{
	return Private::BinaryOpInPlace(Private::FOpModulo(), LhsType, RhsType, Component, OutComponentsConsumed);
}

EValueType Atan2InPlace(EValueType LhsType, EValueType RhsType, TArrayView<FValueComponent> Component, int32& OutComponentsConsumed)
{
	return Private::BinaryOpInPlace(Private::FOpAtan2(), LhsType, RhsType, Component, OutComponentsConsumed);
}

EValueType AppendInPlace(EValueType LhsType, EValueType RhsType, TArrayView<FValueComponent> Component, int32& OutComponentsConsumed)
{
	const FValueTypeDescription& LhsDesc = GetValueTypeDescription(LhsType);
	const FValueTypeDescription& RhsDesc = GetValueTypeDescription(RhsType);
	const int32 NumComponents = FMath::Min<int32>(LhsDesc.NumComponents + RhsDesc.NumComponents, 4);

	OutComponentsConsumed = LhsDesc.NumComponents + RhsDesc.NumComponents - NumComponents;

	if (LhsDesc.ComponentType == RhsDesc.ComponentType)
	{
		return MakeValueType(LhsDesc.ComponentType, NumComponents);
	}
	else if (LhsDesc.ComponentType == EValueComponentType::Double || RhsDesc.ComponentType == EValueComponentType::Double)
	{
		// One of them is double, cast whichever one is not
		const Private::FCastDouble Cast;
		if (LhsDesc.ComponentType != EValueComponentType::Double)
		{
			// Could original LhsDesc.NumComponents have been greater than 4?  Clamp just in case...
			int32 LhsClampedComponents = FMath::Min(LhsDesc.NumComponents, 4);
			for (int32 i = 0; i < LhsClampedComponents; ++i)
			{
				Component[i].Double = Cast(LhsDesc.ComponentType, Component[i]);
			}
		}
		else
		{
			for (int32 i = LhsDesc.NumComponents; i < NumComponents; ++i)
			{
				Component[i].Double = Cast(RhsDesc.ComponentType, Component[i]);
			}
		}
		return MakeValueType(EValueComponentType::Double, NumComponents);
	}
	else if (LhsDesc.ComponentType == EValueComponentType::Float || RhsDesc.ComponentType == EValueComponentType::Float)
	{
		// One of them is float, cast whichever one is not
		const Private::FCastFloat Cast;
		if (LhsDesc.ComponentType != EValueComponentType::Float)
		{
			// Could original LhsDesc.NumComponents have been greater than 4?  Clamp just in case...
			int32 LhsClampedComponents = FMath::Min(LhsDesc.NumComponents, 4);
			for (int32 i = 0; i < LhsClampedComponents; ++i)
			{
				Component[i].Float = Cast(LhsDesc.ComponentType, Component[i]);
			}
		}
		else
		{
			for (int32 i = LhsDesc.NumComponents; i < NumComponents; ++i)
			{
				Component[i].Float = Cast(RhsDesc.ComponentType, Component[i]);
			}
		}
		return MakeValueType(EValueComponentType::Float, NumComponents);
	}
	else
	{
		// Cast everything else to int
		const Private::FCastInt Cast;
		int32 LhsClampedComponents = FMath::Min(LhsDesc.NumComponents, 4);
		for (int32 i = 0; i < LhsClampedComponents; ++i)
		{
			Component[i].Int = Cast(LhsDesc.ComponentType, Component[i]);
		}
		for (int32 i = LhsDesc.NumComponents; i < NumComponents; ++i)
		{
			Component[i].Int = Cast(RhsDesc.ComponentType, Component[i]);
		}
		return MakeValueType(EValueComponentType::Int, NumComponents);
	}
}

} // namespace Shader
} // namespace UE
