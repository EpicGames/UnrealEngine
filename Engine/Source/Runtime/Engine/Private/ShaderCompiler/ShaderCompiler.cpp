// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	ShaderCompiler.cpp: Platform independent shader compilations.
=============================================================================*/

#include "ShaderCompiler.h"
#include "ShaderCompiler/ShaderCompilerInternal.h"
#include "ShaderCompilerPrivate.h"
#include "AsyncCompilationHelpers.h"
#include "AssetCompilingManager.h"
#include "ClearReplacementShaders.h"
#include "ComponentRecreateRenderStateContext.h"
#include "Components/PrimitiveComponent.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "DistributedBuildControllerInterface.h"
#include "EditorSupportDelegates.h"
#include "Engine/RendererSettings.h"
#include "Features/IModularFeatures.h"
#include "Interfaces/IShaderFormat.h"
#include "Interfaces/ITargetPlatform.h"
#include "Interfaces/ITargetPlatformManagerModule.h"
#include "Internationalization/LocKeyFuncs.h"
#include "IOS/IOSPlatformProperties.h"
#include "Logging/StructuredLog.h"
#include "Mac/MacPlatformProperties.h"
#include "MaterialShared.h"
#include "Materials/Material.h"
#include "Materials/MaterialInstance.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/CoreDelegates.h"
#include "Misc/FeedbackContext.h"
#include "Misc/FileHelper.h"
#include "Misc/MessageDialog.h"
#include "Misc/ScopedSlowTask.h"
#include "Modules/ModuleManager.h"
#include "ObjectCacheContext.h"
#include "ProfilingDebugging/CookStats.h"
#include "ProfilingDebugging/DiagnosticTable.h"
#include "ProfilingDebugging/LoadTimeTracker.h"
#include "ProfilingDebugging/StallDetector.h"
#include "RenderUtils.h"
#include "SceneInterface.h"
#include "SceneManagement.h"
#include "Serialization/CompactBinaryWriter.h"
#include "Serialization/MemoryReader.h"
#include "Serialization/NameAsStringProxyArchive.h"
#include "ShaderCodeLibrary.h"
#include "ShaderPlatformConfig.h"
#include "ShaderSerialization.h"
#include "ShaderDiagnostics.h"
#include "ShaderPlatformCachedIniValue.h"
#include "StaticBoundShaderState.h"
#include "StereoRenderUtils.h"
#include "Tasks/Task.h"
#include "UObject/DevObjectVersion.h"
#include "UObject/UObjectIterator.h"
#include "Math/UnitConversion.h"
#include "UnrealEngine.h"
#include "ColorManagement/ColorSpace.h"

#include "SceneTexturesConfig.h"
#include "PSOPrecacheMaterial.h"
#include "PSOPrecacheSettings.h"

#if WITH_EDITOR
#include "Serialization/ArchiveSavePackageDataBuffer.h"
#include "UObject/UObjectGlobals.h"
#include "DerivedDataCache.h"
#include "DerivedDataRequestOwner.h"
#include "TextureCompiler.h"
#include "Rendering/StaticLightingSystemInterface.h"
#include "DirectoryWatcherModule.h"
#include "IDirectoryWatcher.h"
#endif

#if WITH_ODSC
#include "ODSC/ODSCManager.h"
#include "UnrealEngine.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(ShaderCompiler)

#define LOCTEXT_NAMESPACE "ShaderCompiler"

DEFINE_LOG_CATEGORY(LogShaderCompilers);

LLM_DEFINE_TAG(ShaderCompiler);

static TAutoConsoleVariable<bool> CVarRecompileShadersOnSave(
	TEXT("r.ShaderCompiler.RecompileShadersOnSave"),
	false,
	TEXT("When enabled, the editor will attempt to recompile any shader files that have changed when saved.  Useful for iterating on shaders in the editor.\n")
	TEXT("Default: false"),
	ECVF_ReadOnly);

static TAutoConsoleVariable<bool> CVarDebugDumpJobInputHashes(
	TEXT("r.ShaderCompiler.DebugDumpJobInputHashes"),
	false,
	TEXT("If true, the job input hash will be dumped alongside other debug data (in InputHash.txt)"),
	ECVF_ReadOnly);

static TAutoConsoleVariable<bool> CVarDebugDumpJobDiagnostics(
	TEXT("r.ShaderCompiler.DebugDumpJobDiagnostics"),
	false,
	TEXT("If true, all diagnostic messages (errors and warnings) for each shader job will be dumped alongside other debug data (in Diagnostics.txt)"),
	ECVF_ReadOnly);

static TAutoConsoleVariable<bool> CVarDebugDumpShaderCode(
	TEXT("r.ShaderCompiler.DebugDumpShaderCode"),
	false,
	TEXT("If true, each shader job will dump a ShaderCode.bin containing the contents of the output shader code object (the contents of this can differ for each shader format; note that this is the data that is hashed to produce the OutputHash.txt file)"),
	ECVF_ReadOnly);

static TAutoConsoleVariable<bool> CVarDebugDumpShaderCodePlatformHashes(
	TEXT("r.ShaderCompiler.DebugDumpShaderCodePlatformHashes"),
	false,
	TEXT("If true, each shader job will dump a PlatformHash.txt file containing the shader code hash as reported by the platform compiler (if the associated shader format registers this hash with the shader stats).\n")
	TEXT("Note the distinction between this and OutputHash.txt - these files can be used to find shaders which have identical code and only result in different output hashes due to diffs in other metadata."),
	ECVF_ReadOnly);

static TAutoConsoleVariable<bool> CVarDebugDumpDetailedShaderSource(
	TEXT("r.ShaderCompiler.DebugDumpDetailedShaderSource"),
	false,
	TEXT("If true, and if the preprocessed job cache is enabled, this will dump multiple copies of the shader source for any job which has debug output enabled:\n")
	TEXT("\t1. The unmodified output of the preprocessing step as constructed by the PreprocessShader implementation of the IShaderFormat (Preprocessed_<shader>.usf\n")
	TEXT("\t2. The stripped version of the above (with comments, line directives, and whitespace-only lines removed), which is the version hashed for inclusion in the job input hash when the preprocessed job cache is enabled (Stripped_<shader>.usf)")
	TEXT("\t3. The final source as passed to the platform compiler (this will differ if the IShaderFormat compile function applies further modifications to the source after preprocessing; otherwise this will be the same as 2 above (<shader>.usf)\n")
	TEXT("If false, or the preprocessed job cache is disabled, this will simply dump whatever source is passed to the compiler (equivalent to either 1 or 3 depending on if the IShaderFormat implementation modifies the source in the compile step."),
	ECVF_ReadOnly);

static TAutoConsoleVariable<bool> CVarDisableSourceStripping(
	TEXT("r.ShaderCompiler.DisableSourceStripping"),
	false,
	TEXT("If true, the process which strips comments, line directives and whitespace from final preprocessed source is disabled. This results in file associations being maintained and visible in RenderDoc etc., at the cost of less effective deduplication."),
	ECVF_Default);

static TAutoConsoleVariable<bool> CVarAreShaderErrorsFatal(
	TEXT("r.AreShaderErrorsFatal"),
	true,
	TEXT("When enabled, when a the default material or global shaders fail to compile it will issue a Fatal error.  Otherwise just an Error.\n")
	TEXT("Default: true"),
	ECVF_RenderThreadSafe);

int32 GShaderCompilerAllowDistributedCompilation = 1;
static FAutoConsoleVariableRef CVarShaderCompilerAllowDistributedCompilation(
	TEXT("r.ShaderCompiler.AllowDistributedCompilation"),
	GShaderCompilerAllowDistributedCompilation,
	TEXT("If 0, only local (spawned by the engine) ShaderCompileWorkers will be used. If 1, SCWs will be distributed using one of several possible backends (XGE, FASTBuild, SN-DBS)"),
	ECVF_Default
);

int32 GMaxNumDumpedShaderSources = 10;
static FAutoConsoleVariableRef CVarShaderCompilerMaxDumpedShaderSources(
	TEXT("r.ShaderCompiler.MaxDumpedShaderSources"),
	GMaxNumDumpedShaderSources,
	TEXT("Maximum number of preprocessed shader sources to dump as a build artifact on shader compile errors. By default 10."),
	ECVF_ReadOnly
);

int32 GSShaderCheckLevel = 1;
static FAutoConsoleVariableRef CVarGSShaderCheckLevel(
	TEXT("r.Shaders.CheckLevel"),
	GSShaderCheckLevel,
	TEXT("0 => DO_CHECK=0, DO_GUARD_SLOW=0, 1 => DO_CHECK=1, DO_GUARD_SLOW=0, 2 => DO_CHECK=1, DO_GUARD_SLOW=1 for all shaders."),
	ECVF_Default
);

static TAutoConsoleVariable<bool> CVarShaderCompilerDumpDDCKeys(
	TEXT("r.ShaderCompiler.DumpDDCKeys"),
	false,
	TEXT("if != 0, DDC keys for each shadermap will be dumped into project's Saved directory (ShaderDDCKeys subdirectory)"),
	ECVF_Default
);

bool GDebugDumpWorkerCrashLog = false;
static FAutoConsoleVariableRef CVarDebugDumpWorkerCrashLog(
	TEXT("r.ShaderCompiler.DebugDumpWorkerCrashLog"),
	GDebugDumpWorkerCrashLog,
	TEXT("If true, the ShaderCompileWorker will dump its entire log to the Saved folder when a crash is detected."),
	ECVF_ReadOnly
);

static TAutoConsoleVariable<int> CVarShaderCompilerLogSlowJobThreshold(
	TEXT("r.ShaderCompiler.LogSlowJobThreshold"),
	30,
	TEXT("If a single compilation job's compile time exceeds the specified value (in seconds), info about the job will be automatically logged for investigation."),
	ECVF_Default
);


bool GUseMaterialVSUniformBuffer = true;
static FAutoConsoleVariableRef CVarUseMaterialVSUniformBuffer(
	TEXT("r.ShaderCompiler.UseMaterialVSUniformBuffer"),
	GUseMaterialVSUniformBuffer,
	TEXT("If true, translate/compile vertex shaders with a dedicated uniform buffer"),
	ECVF_ReadOnly
);

bool UseMaterialVSUniformBuffer()
{
	return GUseMaterialVSUniformBuffer;
}

bool AreShaderErrorsFatal()
{
	return CVarAreShaderErrorsFatal.GetValueOnAnyThread();
}


namespace ShaderCompiler
{
	FString GetTargetPlatformName(const ITargetPlatform* TargetPlatform)
	{
		if (TargetPlatform)
		{
			return TargetPlatform->PlatformName();
		}

		return TEXT("(current)");
	}

	bool IsRemoteCompilingAllowed()
	{
		// commandline switches override the CVars
		static bool bDisabledFromCommandline = FParse::Param(FCommandLine::Get(), TEXT("NoRemoteShaderCompile"));
		if (bDisabledFromCommandline)
		{
			return false;
		}

		return GShaderCompilerAllowDistributedCompilation != 0;
	}
} // namespace ShaderCompiler

/** Storage for the global shadar map(s) that have been replaced by new one(s), which aren't yet compiled.
 * 
 *	Sometimes a mesh drawing command references a pointer to global SM's memory. To nix these MDCs when we're replacing a global SM, we would just recreate the render state for all the components, but
 *	we may need to access a global shader during such an update, creating a catch 22. So deleting the global SM and updating components is deferred until the new one is compiled. 
 */
FGlobalShaderMap* GGlobalShaderMap_DeferredDeleteCopy[SP_NumPlatforms] = {nullptr};

#if ENABLE_COOK_STATS
namespace GlobalShaderCookStats
{
	FCookStats::FDDCResourceUsageStats UsageStats;
	int32 ShadersCompiled = 0;

	static FCookStatsManager::FAutoRegisterCallback RegisterCookStats([](FCookStatsManager::AddStatFuncRef AddStat)
		{
			UsageStats.LogStats(AddStat, TEXT("GlobalShader.Usage"), TEXT(""));
			AddStat(TEXT("GlobalShader.Misc"), FCookStatsManager::CreateKeyValueArray(
				TEXT("ShadersCompiled"), ShadersCompiled
			));
		});
}
#endif

const FGuid& GetGlobalShaderMapDDCGuid()
{
	static FGuid GlobalShaderMapDDCGuid = FDevSystemGuids::GetSystemGuid(FDevSystemGuids::Get().GLOBALSHADERMAP_DERIVEDDATA_VER);
	return GlobalShaderMapDDCGuid;
}

const FGuid& GetMaterialShaderMapDDCGuid()
{
	static FGuid MaterialShaderMapDDCGuid = FDevSystemGuids::GetSystemGuid(FDevSystemGuids::Get().MATERIALSHADERMAP_DERIVEDDATA_VER);
	return MaterialShaderMapDDCGuid;
}

bool ShouldDumpShaderDDCKeys()
{
	return CVarShaderCompilerDumpDDCKeys.GetValueOnAnyThread();
}

void DumpShaderDDCKeyToFile(const EShaderPlatform InPlatform, bool bWithEditor, const FString& FileName, const FString& DDCKey)
{
	// deprecated version
	const FString SubDirectory = bWithEditor ? TEXT("Editor") : TEXT("Game");
	const FString TempPath = FPaths::ProjectSavedDir() / TEXT("ShaderDDCKeys") / SubDirectory / LexToString(InPlatform);
	IFileManager::Get().MakeDirectory(*TempPath, true);

	const FString TempFile = TempPath / FileName;

	TUniquePtr<FArchive> DumpAr(IFileManager::Get().CreateFileWriter(*TempFile));
	// serializing the string via << produces a non-textual file because it saves string's length, too
	DumpAr->Serialize(const_cast<TCHAR*>(*DDCKey), DDCKey.Len() * sizeof(TCHAR));
}

void DumpShaderDDCKeyToFile(const EShaderPlatform InPlatform, bool bEditorOnly, const TCHAR* DebugGroupName, const FString& DDCKey)
{
	const FString FileName = FString::Printf(TEXT("DDCKey-%s.txt"), bEditorOnly ? TEXT("Editor") : TEXT("Game"));

	const FString TempPath = GShaderCompilingManager->GetAbsoluteShaderDebugInfoDirectory() / FGenericDataDrivenShaderPlatformInfo::GetName(InPlatform).ToString() / DebugGroupName;
	IFileManager::Get().MakeDirectory(*TempPath, true);

	const FString TempFile = TempPath / FileName;

	FFileHelper::SaveStringToFile(DDCKey, *TempFile);
}


static float GRegularWorkerTimeToLive = 20.0f;
static float GBuildWorkerTimeToLive = 600.0f;


template<class EnumType>
constexpr auto& CastEnumToUnderlyingTypeReference(EnumType& Type)
{
	static_assert(TIsEnum<EnumType>::Value, "");
	using UnderType = __underlying_type(EnumType);
	return reinterpret_cast<UnderType&>(Type);
}

// Set to 1 to debug ShaderCompileWorker.exe. Set a breakpoint in LaunchWorker() to get the cmd-line.
#define DEBUG_SHADERCOMPILEWORKER 0

// Default value comes from bPromptToRetryFailedShaderCompiles in BaseEngine.ini
// This is set as a global variable to allow changing in the debugger even in release
// For example if there are a lot of content shader compile errors you want to skip over without relaunching
bool GRetryShaderCompilation = true;

static FShaderCompilingManager::EDumpShaderDebugInfo GDumpShaderDebugInfo = FShaderCompilingManager::EDumpShaderDebugInfo::OnError;
static FAutoConsoleVariableRef CVarDumpShaderDebugInfo(
	TEXT("r.DumpShaderDebugInfo"),
	CastEnumToUnderlyingTypeReference(GDumpShaderDebugInfo),
	TEXT("Dumps debug info for compiled shaders to GameName/Saved/ShaderDebugInfo\n")
	TEXT("When set to 1, debug info is dumped for all compiled shader\n")
	TEXT("When set to 2, it is restricted to shaders with compilation errors\n")
	TEXT("When set to 3, it is restricted to shaders with compilation errors or warnings\n")
	TEXT("The debug info is platform dependent, but usually includes a preprocessed version of the shader source.\n")
	TEXT("Global shaders automatically dump debug info if r.ShaderDevelopmentMode is enabled, this cvar is not necessary.\n")
	TEXT("On iOS, if the PowerVR graphics SDK is installed to the default path, the PowerVR shader compiler will be called and errors will be reported during the cook.")
	);

static int32 GDumpShaderDebugInfoShort = 0;
static FAutoConsoleVariableRef CVarDumpShaderDebugShortNames(
	TEXT("r.DumpShaderDebugShortNames"),
	GDumpShaderDebugInfoShort,
	TEXT("Only valid when r.DumpShaderDebugInfo > 0.\n")
	TEXT("When set to 1, will shorten names factory and shader type folder names to avoid issues with long paths.")
	);

static int32 GDumpShaderDebugInfoBindless = 0;
static FAutoConsoleVariableRef CVarDumpShaderDebugBindlessNames(
	TEXT("r.DumpShaderDebugBindlessNames"),
	GDumpShaderDebugInfoBindless,
	TEXT("Only valid when r.DumpShaderDebugInfo > 0.\n")
	TEXT("When set to 1, will add bindless folder names.")
	);

static int32 GShaderMapCompilationTimeout = 2 * 60 * 60;	// anything below an hour can hit a false positive
static FAutoConsoleVariableRef CVarShaderMapCompilationTimeout(
	TEXT("r.ShaderCompiler.ShadermapCompilationTimeout"),
	GShaderMapCompilationTimeout,
	TEXT("Maximum number of seconds a single shadermap (which can be comprised of multiple jobs) can be compiled after being considered hung.")
);

static int32 GCrashOnHungShaderMaps = 0;
static FAutoConsoleVariableRef CVarCrashOnHungShaderMaps(
	TEXT("r.ShaderCompiler.CrashOnHungShaderMaps"),
	GCrashOnHungShaderMaps,
	TEXT("If set to 1, the shader compiler will crash on hung shadermaps.")
);

static int32 GForceAllCoresForShaderCompiling = 0;
static FAutoConsoleVariableRef CVarForceAllCoresForShaderCompiling(
	TEXT("r.ForceAllCoresForShaderCompiling"),
	GForceAllCoresForShaderCompiling,
	TEXT("When set to 1, it will ignore INI settings and launch as many ShaderCompileWorker instances as cores are available.\n")
	TEXT("Improves shader throughput but for big projects it can make the machine run OOM")
);

static TAutoConsoleVariable<int32> CVarShadersSymbols(
	TEXT("r.Shaders.Symbols"),
	0,
	TEXT("Enables debugging of shaders in platform specific graphics debuggers. This will generate and write shader symbols.\n")
	TEXT("This enables the behavior of both r.Shaders.GenerateSymbols and r.Shaders.WriteSymbols.\n")
	TEXT("Enables shader debugging features that require shaders to be recompiled. This compiles shaders with symbols and also includes extra runtime information like shader names. When using graphical debuggers it can be useful to enable this on startup.\n")
	TEXT("This setting can be overriden in any Engine.ini under the [ShaderCompiler] section."),
	ECVF_ReadOnly);

static TAutoConsoleVariable<int32> CVarShadersSymbolsInfo(
	TEXT("r.Shaders.SymbolsInfo"),
	0,
	TEXT("In lieu of a full set of platform shader PDBs, save out a slimmer ShaderSymbols.Info which contains shader platform hashes and shader debug info.\n")
	TEXT("An option for when it is not practical to save PDBs for shaders all the time.\n")
	TEXT("This setting can be overriden in any Engine.ini under the [ShaderCompiler] section."),
	ECVF_ReadOnly);

static TAutoConsoleVariable<int32> CVarShadersGenerateSymbols(
	TEXT("r.Shaders.GenerateSymbols"),
	0,
	TEXT("Enables generation of data for shader debugging when compiling shaders. This explicitly does not write any shader symbols to disk.\n")
	TEXT("This setting can be overriden in any Engine.ini under the [ShaderCompiler] section."),
	ECVF_ReadOnly);

static TAutoConsoleVariable<int32> CVarShadersWriteSymbols(
	TEXT("r.Shaders.WriteSymbols"),
	0,
	TEXT("Enables writing shader symbols to disk for platforms that support that. This explicitly does not enable generation of shader symbols.\n")
	TEXT("This setting can be overriden in any Engine.ini under the [ShaderCompiler] section."),
	ECVF_ReadOnly);

static TAutoConsoleVariable<FString> CVarShadersSymbolPathOverride(
	TEXT("r.Shaders.SymbolPathOverride"),
	"",
	TEXT("Override output location of shader symbols. If the path contains the text '{Platform}', that will be replaced with the shader platform string.\n")
	TEXT("Empty: use default location Saved/ShaderSymbols/{Platform}\n")
	TEXT("This setting can be overriden in any Engine.ini under the [ShaderCompiler] section."),
	ECVF_ReadOnly);

static TAutoConsoleVariable<FString> CVarShadersSymbolFileNameOverride(
	TEXT("r.Shaders.SymbolFileNameOverride"),
	"",
	TEXT("Override base file name for shader symbol related aggregate outputs (.zip, .info).\n")
	TEXT("'{Platform}' will be replaced with the shader platform string.\n")
	TEXT("'{DLC}' will be replaced with the DLC name if there is one.\n")
	TEXT("'{?DLC-}' will be replaced with a dash if there is DLC, for formatting.\n")
	TEXT("Empty: use default 'ShaderSymbols'\n")
	TEXT("This setting can be overridden in any Engine.ini under the [ShaderCompiler] section."),
	ECVF_ReadOnly);

static TAutoConsoleVariable<FString> CVarShaderTypeStatsFileNameOverride(
	TEXT("r.Shaders.ShaderTypeStatsFileNameOverride"),
	"",
	TEXT("Override base file name for shader type outputs.\n")
	TEXT("'{Platform}' will be replaced with the shader platform string.\n")
	TEXT("'{DLC}' will be replaced with the DLC name if there is one.\n")
	TEXT("'{?DLC-}' will be replaced with a dash if there is DLC, for formatting.\n")
	TEXT("Empty: use default 'ShaderTypeStats-{Platform}'\n")
	TEXT("This setting can be overridden in any Engine.ini under the [ShaderCompiler] section."),
	ECVF_ReadOnly);

static TAutoConsoleVariable<int32> CVarAllowUniqueDebugInfo(
	TEXT("r.Shaders.AllowUniqueSymbols"),
	0,
	TEXT("When enabled, this tells supported shader compilers to generate symbols based on source files.\n")
	TEXT("Enabling this can cause a drastic increase in the number of symbol files, enable only if absolutely necessary.\n")
	TEXT("This setting can be overriden in any Engine.ini under the [ShaderCompiler] section."),
	ECVF_ReadOnly);

static TAutoConsoleVariable<int32> CVarShadersWriteSymbolsZip(
	TEXT("r.Shaders.WriteSymbols.Zip"),
	0,
	TEXT(" 0: Export as loose files.\n")
	TEXT(" 1: Export as an uncompressed archive.\n")
	TEXT(" 2: Export as a compressed archive.\n"),
	ECVF_ReadOnly);

static TAutoConsoleVariable<int32> CVarShadersEnableExtraData(
	TEXT("r.Shaders.ExtraData"),
	0,
	TEXT("Enables generation of extra shader data that can be used at runtime. This includes shader names and other platform specific data.\n")
	TEXT("This can add bloat to compiled shaders and can prevent shaders from being deduplicated.\n")
	TEXT("This setting can be overriden in any Engine.ini under the [ShaderCompiler] section."),
	ECVF_ReadOnly);

static TAutoConsoleVariable<int32> CVarOptimizeShaders(
	TEXT("r.Shaders.Optimize"),
	1,
	TEXT("Whether to optimize shaders.  When using graphical debuggers like Nsight it can be useful to disable this on startup.\n")
	TEXT("This setting can be overriden in any Engine.ini under the [ShaderCompiler] section."),
	ECVF_ReadOnly);

static TAutoConsoleVariable<int32> CVarShaderFastMath(
	TEXT("r.Shaders.FastMath"),
	1,
	TEXT("Whether to use fast-math optimisations in shaders."),
	ECVF_ReadOnly);

static TAutoConsoleVariable<int32> CVarShaderZeroInitialise(
	TEXT("r.Shaders.ZeroInitialise"),
	1,
	TEXT("Whether to enforce zero initialise local variables of primitive type in shaders. Defaults to 1 (enabled). Not all shader languages can omit zero initialisation."),
	ECVF_ReadOnly);

static TAutoConsoleVariable<int32> CVarShaderBoundsChecking(
	TEXT("r.Shaders.BoundsChecking"),
	1,
	TEXT("Whether to enforce bounds-checking & flush-to-zero/ignore for buffer reads & writes in shaders. Defaults to 1 (enabled). Not all shader languages can omit bounds checking."),
	ECVF_ReadOnly);

static TAutoConsoleVariable<int32> CVarShaderWarningsAsErrors(
	TEXT("r.Shaders.WarningsAsErrors"),
	0,
	TEXT("Whether to treat warnings as errors when compiling shaders. (0: disabled (default), 1: global shaders only, 2: all shaders)). This setting may be ignored on older platforms."),
	ECVF_ReadOnly);

static TAutoConsoleVariable<int32> CVarShaderFlowControl(
	TEXT("r.Shaders.FlowControlMode"),
	0,
	TEXT("Specifies whether the shader compiler should preserve or unroll flow-control in shader code.\n")
	TEXT("This is primarily a debugging aid and will override any per-shader or per-material settings if not left at the default value (0).\n")
	TEXT("\t0: Off (Default) - Entirely at the discretion of the platform compiler or the specific shader/material.\n")
	TEXT("\t1: Prefer - Attempt to preserve flow-control.\n")
	TEXT("\t2: Avoid - Attempt to unroll and flatten flow-control.\n"),
	ECVF_ReadOnly);

static TAutoConsoleVariable<int32> CVarD3DCheckedForTypedUAVs(
	TEXT("r.D3D.CheckedForTypedUAVs"),
	1,
	TEXT("Whether to disallow usage of typed UAV loads, as they are unavailable in Windows 7 D3D 11.0.\n")
	TEXT(" 0: Allow usage of typed UAV loads.\n")
	TEXT(" 1: Disallow usage of typed UAV loads. (default)"),
	ECVF_ReadOnly);

static TAutoConsoleVariable<int32> CVarD3DForceDXC(
	TEXT("r.D3D.ForceDXC"),
	0,
	TEXT("Forces DirectX Shader Compiler (DXC) to be used for all D3D shaders. Shaders compiled with this option are only compatible with D3D12.\n")
	TEXT(" 0: Disable (default)\n")
	TEXT(" 1: Force new compiler for all shaders"),
	ECVF_ReadOnly);

static TAutoConsoleVariable<int32> CVarWarpCulling(
	TEXT("r.WarpCulling"),
	0,
	TEXT("Enable Warp Culling optimization for platforms that support it.\n")
	TEXT(" 0: Disable (default)\n")
	TEXT(" 1: Enable"),
	ECVF_ReadOnly);

static TAutoConsoleVariable<int32> CVarCullBeforeFetch(
	TEXT("r.CullBeforeFetch"),
	0,
	TEXT("Enable Cull-Before-Fetch optimization for platforms that support it.\n")
	TEXT(" 0: Disable (default)\n")
	TEXT(" 1: Enable"),
	ECVF_ReadOnly);

int32 GCreateShadersOnLoad = 0;
static FAutoConsoleVariableRef CVarCreateShadersOnLoad(
	TEXT("r.CreateShadersOnLoad"),
	GCreateShadersOnLoad,
	TEXT("Whether to create shaders on load, which can reduce hitching, but use more memory.  Otherwise they will be created as needed."));

static TAutoConsoleVariable<bool> CVarForceSpirvDebugInfo(
	TEXT("r.ShaderCompiler.ForceSpirvDebugInfo"),
	false,
	TEXT("Enable SPIR-V specific debug information independently of debug and optimization compilation options.\n")
	TEXT(" false: Disable (default)\n")
	TEXT(" true: Enable"),
	ECVF_ReadOnly);

bool CreateShadersOnLoad()
{
	return GCreateShadersOnLoad != 0;
}

static TAutoConsoleVariable<int32> CVarShadersValidation(
	TEXT("r.Shaders.Validation"),
	1,
	TEXT("Enabled shader compiler validation warnings and errors."),
	ECVF_ReadOnly);

static TAutoConsoleVariable<int32> CVarShadersRemoveDeadCode(
	TEXT("r.Shaders.RemoveDeadCode"),
	1,
	TEXT("Run a preprocessing step that removes unreferenced code before compiling shaders.\n")
	TEXT("This can improve the compilation speed for shaders which include many large utility headers.\n")
	TEXT("\t0: Keep all input source code.\n")
	TEXT("\t1: Remove unreferenced code before compilation (Default)\n"),
	ECVF_ReadOnly);

namespace ShaderCompiler
{
	bool IsDumpShaderDebugInfoAlwaysEnabled()
	{
		return GDumpShaderDebugInfo != FShaderCompilingManager::EDumpShaderDebugInfo::Always;
	}

} // namespace ShaderCompiler

#if ENABLE_COOK_STATS
namespace ShaderCompilerCookStats
{
	static std::atomic<double> BlockingTimeSec = 0.0;
	static std::atomic<double> GlobalBeginCompileShaderTimeSec = 0.0;
	static std::atomic<int32> GlobalBeginCompileShaderCalls = 0;
	static std::atomic<double> ProcessAsyncResultsTimeSec = 0.0;
	std::atomic<double> AsyncCompileTimeSec = 0.0;

	static FCookStatsManager::FAutoRegisterCallback RegisterCookStats([](FCookStatsManager::AddStatFuncRef AddStat)
		{
			AddStat(TEXT("ShaderCompiler"), FCookStatsManager::CreateKeyValueArray(
				TEXT("BlockingTimeSec"), BlockingTimeSec.load(std::memory_order_relaxed),
				TEXT("AsyncCompileTimeSec"), AsyncCompileTimeSec.load(std::memory_order_relaxed),
				TEXT("GlobalBeginCompileShaderTimeSec"), GlobalBeginCompileShaderTimeSec.load(std::memory_order_relaxed),
				TEXT("GlobalBeginCompileShaderCalls"), GlobalBeginCompileShaderCalls.load(std::memory_order_relaxed),
				TEXT("ProcessAsyncResultsTimeSec"), ProcessAsyncResultsTimeSec.load(std::memory_order_relaxed)
			));
		});
}
#endif

#if WITH_EDITOR
static bool CheckSingleJob(const FShaderCompileJob& SingleJob, TArray<FString>& OutErrors)
{
	if (SingleJob.bSucceeded)
	{
		checkf(SingleJob.Output.ShaderCode.GetShaderCodeSize() > 0, TEXT("Abnormal shader code size for a successful job: %d bytes"), SingleJob.Output.ShaderCode.GetShaderCodeSize());
	}

	bool bSucceeded = SingleJob.bSucceeded;

	if (SingleJob.Key.ShaderType)
	{
		// Allow the shader validation to fail the compile if it sees any parameters bound that aren't supported.
		const bool bValidationResult = SingleJob.Key.ShaderType->ValidateCompiledResult(
			(EShaderPlatform)SingleJob.Input.Target.Platform,
			SingleJob.Output.ParameterMap,
			OutErrors);
		bSucceeded = bValidationResult && bSucceeded;
	}

	if (SingleJob.Key.VFType)
	{
		const int32 OriginalNumErrors = OutErrors.Num();

		// Allow the vertex factory to fail the compile if it sees any parameters bound that aren't supported
		SingleJob.Key.VFType->ValidateCompiledResult((EShaderPlatform)SingleJob.Input.Target.Platform, SingleJob.Output.ParameterMap, OutErrors);

		if (OutErrors.Num() > OriginalNumErrors)
		{
			bSucceeded = false;
		}
	}

	return bSucceeded;
};
#endif // WITH_EDITOR

FShaderCompilingManager* GShaderCompilingManager = nullptr;

bool FShaderCompilingManager::AllTargetPlatformSupportsRemoteShaderCompiling()
{
	// no compiling support
	if (!AllowShaderCompiling())
	{
		return false;
	}

	ITargetPlatformManagerModule* TPM = GetTargetPlatformManager();	
	if (!TPM)
	{
		return false;
	}
	
	const TArray<ITargetPlatform*>& Platforms = TPM->GetActiveTargetPlatforms();
	for (int32 Index = 0; Index < Platforms.Num(); Index++)
	{
		if (!Platforms[Index]->CanSupportRemoteShaderCompile())
		{
			return false;
		}
	}
	
	return true;
}

// Returns a rank for the preference of distributed shader controllers; Higher is better.
static int32 GetShaderControllerPreferenceRank(IDistributedBuildController& Controller)
{
	const FString Name = Controller.GetName();
	if (Name.StartsWith(TEXT("UBA")))
	{
		return 2;
	}
	else if (Name.StartsWith(TEXT("XGE")))
	{
		return 1;
	}
	else
	{
		return 0;
	}
}

IDistributedBuildController* FShaderCompilingManager::FindRemoteCompilerController() const
{
	// no controllers needed if not compiling
	if (!AllowShaderCompiling())
	{
		return nullptr;
	}

	TArray<IDistributedBuildController*> AvailableControllers = IModularFeatures::Get().GetModularFeatureImplementations<IDistributedBuildController>(IDistributedBuildController::GetModularFeatureType());

	// Prefer UBA, then XGE, and fallback to any other controller otherwise
	int32 SupportedControllerPreferenceRank = 0;
	IDistributedBuildController* SupportedController = nullptr;

	for (IDistributedBuildController* Controller : AvailableControllers)
	{
		if (Controller != nullptr && Controller->IsSupported())
		{
			const int32 PreferenceRank = GetShaderControllerPreferenceRank(*Controller);
			if (SupportedController == nullptr || SupportedControllerPreferenceRank < PreferenceRank)
			{
				SupportedController = Controller;
				SupportedControllerPreferenceRank = PreferenceRank;
			}
		}
	}

	if (SupportedController != nullptr)
	{
		SupportedController->InitializeController();
		return SupportedController;
	}

	return nullptr;
}

void FShaderCompilingManager::ReportMemoryUsage()
{
	// This function runs from within an OOM callback. It should not take locks, as much as possible.
	constexpr bool bAllowToWaitForLock = false;
	for (const TUniquePtr<FShaderCompileThreadRunnableBase>& ThreadPtr : Threads)
	{
		ThreadPtr->PrintWorkerMemoryUsage(bAllowToWaitForLock);
	}
}


static bool FindShaderCompileWorkerExecutableInLaunchDir(const FString& ExecutableName, FString& OutFilename)
{
	FString LocalShaderCompileWorkerName = FPaths::Combine(FPaths::LaunchDir(), ExecutableName);
	if (!IFileManager::Get().FileExists(*LocalShaderCompileWorkerName))
	{
		LocalShaderCompileWorkerName = FPaths::Combine(FPaths::LaunchDir(), TEXT("../../../Engine/Binaries"), FPlatformProcess::GetBinariesSubdirectory(), ExecutableName);

		if (!IFileManager::Get().FileExists(*LocalShaderCompileWorkerName))
		{			
			return false;
		}
	}

	OutFilename = LocalShaderCompileWorkerName;
	return true;
}

FShaderCompilingManager::FShaderCompilingManager() :
	bCompilingDuringGame(false),
	NumExternalJobs(0),
	AllJobs(CompileQueueSection),
	NumSingleThreadedRunsBeforeRetry(GSingleThreadedRunsIdle),
	SuppressedShaderPlatforms(0),
	BuildDistributionController(nullptr),
	bNoShaderCompilation(false),
	bAllowForIncompleteShaderMaps(false),
	Notification(MakeUnique<FAsyncCompilationNotification>(GetAssetNameFormat()))
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FShaderCompilingManager::FShaderCompilingManager);

	// don't perform any initialization if compiling is not allowed
	if (!AllowShaderCompiling())
	{
		// use existing flag to disable compiling
		bNoShaderCompilation = true;
		return;
	}

	bIsEngineLoopInitialized = false;
	FCoreDelegates::OnFEngineLoopInitComplete.AddLambda([&]() 
		{ 
			bIsEngineLoopInitialized = true; 
		}
	);

	WorkersBusyTime = 0;

#if PLATFORM_WINDOWS
#if PLATFORM_WINDOWS_ARM64EC
	FString ExecutableName("ShaderCompileWorkerarm64ec.exe");
#elif PLATFORM_CPU_ARM_FAMILY
	FString ExecutableName("ShaderCompileWorkerarm64.exe");
#else
	FString ExecutableName("ShaderCompileWorker.exe");
#endif
#else
	FString ExecutableName("ShaderCompileWorker");
#endif

	// first look for project-specific version
	ShaderCompileWorkerName = FPaths::Combine(FPaths::ProjectDir(), TEXT("Binaries"), FPlatformProcess::GetBinariesSubdirectory(), ExecutableName);
	if (!IFileManager::Get().FileExists(*ShaderCompileWorkerName))
	{
		// fallback to standard Engine location
		ShaderCompileWorkerName = FPaths::Combine(FPaths::EngineDir(), TEXT("Binaries"), FPlatformProcess::GetBinariesSubdirectory(), ExecutableName);
	}

	// Optionally allow the shader worker path to use the launch directory, this allows the engine to use a locally built shader compile worker when running with the -basedir argument
	bool bUseShaderCompilerFromLaunchDir = false;
	if (GConfig->GetBool(TEXT("DevOptions.Shaders"), TEXT("bUseShaderCompilerFromLaunchDir"), bUseShaderCompilerFromLaunchDir, GEngineIni) && bUseShaderCompilerFromLaunchDir)
	{
		FString LocalShaderCompileWorkerName;
		if (FindShaderCompileWorkerExecutableInLaunchDir(ExecutableName, LocalShaderCompileWorkerName))
		{
			ShaderCompileWorkerName = LocalShaderCompileWorkerName;
		}
		else
		{
			UE_LOGF(LogShaderCompilers, Warning, "Using bUseShaderCompilerFromLaunchDir but could not find shader compile worker in LaunchDir - '%ls'.", *FPaths::LaunchDir());
		}
	}

	// Threads must use absolute paths on Windows in case the current directory is changed on another thread!
	ShaderCompileWorkerName = FPaths::ConvertRelativePathToFull(ShaderCompileWorkerName);

	// Read values from the engine ini
	verify(GConfig->GetBool( TEXT("DevOptions.Shaders"), TEXT("bAllowCompilingThroughWorkers"), bAllowCompilingThroughWorkers, GEngineIni ));
	verify(GConfig->GetBool( TEXT("DevOptions.Shaders"), TEXT("bAllowAsynchronousShaderCompiling"), bAllowAsynchronousShaderCompiling, GEngineIni ));

	// Explicitly load ShaderPreprocessor module so it will run its initialization step
	FModuleManager::LoadModuleChecked<IModuleInterface>(TEXT("ShaderPreprocessor"));
	
	// override the use of workers, can be helpful for debugging shader compiler code
	static const IConsoleVariable* CVarAllowCompilingThroughWorkers = IConsoleManager::Get().FindConsoleVariable(TEXT("r.Shaders.AllowCompilingThroughWorkers"), false);
	if (!FPlatformProcess::SupportsMultithreading() || FParse::Param(FCommandLine::Get(), TEXT("noshaderworker")) || (CVarAllowCompilingThroughWorkers && CVarAllowCompilingThroughWorkers->GetInt() == 0))
	{
		bAllowCompilingThroughWorkers = false;
	}

	if (!FPlatformProcess::SupportsMultithreading())
	{
		bAllowAsynchronousShaderCompiling = false;
	}

	verify(GConfig->GetInt( TEXT("DevOptions.Shaders"), TEXT("MaxShaderJobBatchSize"), MaxShaderJobBatchSize, GEngineIni ));
	verify(GConfig->GetBool( TEXT("DevOptions.Shaders"), TEXT("bPromptToRetryFailedShaderCompiles"), bPromptToRetryFailedShaderCompiles, GEngineIni ));
	verify(GConfig->GetBool(TEXT("DevOptions.Shaders"), TEXT("bDebugBreakOnPromptToRetryShaderCompile"), bDebugBreakOnPromptToRetryShaderCompile, GEngineIni));
	verify(GConfig->GetBool( TEXT("DevOptions.Shaders"), TEXT("bLogJobCompletionTimes"), bLogJobCompletionTimes, GEngineIni ));
	GConfig->GetFloat(TEXT("DevOptions.Shaders"), TEXT("WorkerTimeToLive"), GRegularWorkerTimeToLive, GEngineIni);
	GConfig->GetFloat(TEXT("DevOptions.Shaders"), TEXT("BuildWorkerTimeToLive"), GBuildWorkerTimeToLive, GEngineIni);

	verify(GConfig->GetFloat( TEXT("DevOptions.Shaders"), TEXT("ProcessGameThreadTargetTime"), ProcessGameThreadTargetTime, GEngineIni ));

#if UE_BUILD_DEBUG
	// Increase budget for processing results in debug or else it takes forever to finish due to poor framerate
	ProcessGameThreadTargetTime *= 3;
#endif

	// Get the current process Id, this will be used by the worker app to shut down when it's parent is no longer running.
	ProcessId = FPlatformProcess::GetCurrentProcessId();

	// Use a working directory unique to this game, process and thread so that it will not conflict 
	// With processes from other games, processes from the same game or threads in this same process.
	// Use IFileManager to do path conversion to properly handle sandbox paths (outside of standard paths in particular).
	//ShaderBaseWorkingDirectory = FPlatformProcess::ShaderWorkingDir() / FString::FromInt(ProcessId) + TEXT("/");

	{
		FGuid Guid;
		Guid = FGuid::NewGuid();
		FString LegacyShaderWorkingDirectory = FPaths::ProjectIntermediateDir() / TEXT("Shaders/WorkingDirectory/")  / FString::FromInt(ProcessId) + TEXT("/");
		ShaderBaseWorkingDirectory = FPaths::ShaderWorkingDir() / *Guid.ToString(EGuidFormats::Digits) + TEXT("/");
		UE_LOGF(LogShaderCompilers, Log, "Guid format shader working directory is %d characters bigger than the processId version (%ls).", ShaderBaseWorkingDirectory.Len() - LegacyShaderWorkingDirectory.Len(), *LegacyShaderWorkingDirectory );
	}

	if (!IFileManager::Get().DeleteDirectory(*ShaderBaseWorkingDirectory, false, true))
	{
		UE_LOGF(LogShaderCompilers, Fatal, "Could not delete the shader compiler working directory '%ls'.", *ShaderBaseWorkingDirectory);
	}
	else
	{
		UE_LOGF(LogShaderCompilers, Log, "Cleaned the shader compiler working directory '%ls'.", *ShaderBaseWorkingDirectory);
	}
	FString AbsoluteBaseDirectory = IFileManager::Get().ConvertToAbsolutePathForExternalAppForWrite(*ShaderBaseWorkingDirectory);
	FPaths::NormalizeDirectoryName(AbsoluteBaseDirectory);
	AbsoluteShaderBaseWorkingDirectory = AbsoluteBaseDirectory + TEXT("/");

	// Initialize the shader debug info path; this internally uses a local static var so we create it as early as possible in the init loop to avoid thread safety issues 
	GetShaderDebugInfoPath();

	CalculateNumberOfCompilingThreads(FPlatformMisc::NumberOfCores(), FPlatformMisc::NumberOfCoresIncludingHyperthreads());

	// Launch local and remote shader compiling threads
	{
		constexpr bool bDelayCompileThreadsExecution = true;
		FShaderCompileThreadRunnableBase* RemoteCompileThread = LaunchRemoteShaderCompilingThread(bDelayCompileThreadsExecution);

		GConfig->SetBool(TEXT("/Script/UnrealEd.UnrealEdOptions"), TEXT("UsingXGE"), RemoteCompileThread != nullptr, GEditorIni);

		if (!bUseOnlyDistributedCompilationThread)
		{
			LaunchLocalShaderCompilingThread(bDelayCompileThreadsExecution);
		}

		for (const auto& Thread : Threads)
		{
			Thread->StartThread();
		}
	}

	OutOfMemoryDelegateHandle = FCoreDelegates::GetOutOfMemoryDelegate().AddRaw(this, &FShaderCompilingManager::ReportMemoryUsage);

	FAssetCompilingManager::Get().RegisterManager(this);

	// Ensure directory for dumping worker crash log exits before launching workers
	if (GDebugDumpWorkerCrashLog)
	{
		FString CustomCrashLogsDir;
		if (FParse::Value(FCommandLine::Get(), TEXT("ShaderCompileWorkerCrashLogsDir="), CustomCrashLogsDir))
		{
			WorkerCrashLogBaseDirectory = MoveTemp(CustomCrashLogsDir);
		}
		else if (GIsBuildMachine)
		{
			WorkerCrashLogBaseDirectory = GetBuildMachineArtifactBasePath();
		}

		// If this is empty, fall back to relative paths and the default log directory
		if (!WorkerCrashLogBaseDirectory.IsEmpty() && !IFileManager::Get().DirectoryExists(*WorkerCrashLogBaseDirectory))
		{
			if (!IFileManager::Get().MakeDirectory(*WorkerCrashLogBaseDirectory, true))
			{
				const uint32 ErrorCode = FPlatformMisc::GetLastError();
				UE_LOGF(LogShaderCompilers, Warning, "Failed to create directory for ShaderCompileWorker crash logs '%ls' (Error Code: %u)", *WorkerCrashLogBaseDirectory, ErrorCode);
			}
		}
	}

#if WITH_EDITOR
	static const bool bAllowShaderRecompileOnSave = CVarRecompileShadersOnSave.GetValueOnAnyThread();
	if (bAllowShaderRecompileOnSave)
	{
		if (IDirectoryWatcher* DirectoryWatcher = FModuleManager::LoadModuleChecked<FDirectoryWatcherModule>(TEXT("DirectoryWatcher")).Get())
		{
			// Handle if we are watching a directory for changes.
			{
				UE_LOGF(LogShaderCompilers, Display, "Register directory watchers for shader files.");

				const TMap<FString, FString>& ShaderSourceDirectoryMappings = AllShaderSourceDirectoryMappings();

				DirectoryWatcherHandles.Reserve(ShaderSourceDirectoryMappings.Num());

				for (const auto& It : ShaderSourceDirectoryMappings)
				{
					FString DirectoryToWatch = It.Value;
					if (FPaths::IsRelative(DirectoryToWatch))
					{
						DirectoryToWatch = FPaths::ConvertRelativePathToFull(DirectoryToWatch);
					}

					FDelegateHandle& DirectoryWatcherHandle = DirectoryWatcherHandles.Add(DirectoryToWatch, FDelegateHandle());

					DirectoryWatcher->RegisterDirectoryChangedCallback_Handle(
						DirectoryToWatch,
						IDirectoryWatcher::FDirectoryChanged::CreateLambda([](const TArray<FFileChangeData>& InFileChangeDatas) {

							TRACE_CPUPROFILER_EVENT_SCOPE(HandleDirectoryChanged);

							if (!bAllowShaderRecompileOnSave)
							{
								return;
							}

							TArray<FString> ChangedShaderFiles;
							for (const FFileChangeData& It : InFileChangeDatas)
							{
								if (It.Filename.EndsWith(TEXT(".usf")) || It.Filename.EndsWith(TEXT(".ush")) || It.Filename.EndsWith(TEXT(".h")))
								{
									UE_LOGF(LogShaderCompilers, Display, "Detected change on %ls", *It.Filename);

									ChangedShaderFiles.AddUnique(It.Filename);
								}
							}

							if (ChangedShaderFiles.Num())
							{
								// Mappings from:
								// Key:   /Engine to
								// Value: ../../../Engine/Shaders
								const TMap<FString, FString>& ShaderSourceDirectoryMappings = AllShaderSourceDirectoryMappings();

								FString RemappedShaderFileName;
								for (const auto& It : ShaderSourceDirectoryMappings)
								{
									// ChangedShaderFiles will be of format: ../../../Engine/Shaders/Private/PostProcessGBufferHints.usf
									if (ChangedShaderFiles[0].StartsWith(It.Value))
									{
										// Change from relative path to Engine absolute path.
										// i.e. change `../../../Engine/Shaders/Private/PostProcessGBufferHints.usf` to `/Engine/Shaders/Private/PostProcessGBufferHints.usf`
										RemappedShaderFileName = ChangedShaderFiles[0].Replace(*It.Value, *It.Key);
									}
								}

								// Issue a `recompileshaders /Engine/Shaders/Private/PostProcessGBufferHints.usf` command, which will just compile that shader source file.
								RecompileShaders(*RemappedShaderFileName, *GLog);

								UE_LOGF(LogShaderCompilers, Display, "Ready for new shader file changes");
							}
						}),
						DirectoryWatcherHandle);

					if (DirectoryWatcherHandle.IsValid())
					{
						UE_LOGF(LogShaderCompilers, Display, "Watching %ls -> %ls", *It.Key, *DirectoryToWatch);
					}
					else
					{
						UE_LOGF(LogShaderCompilers, Error, "Failed to set up directory watcher %ls -> %ls", *It.Key, *DirectoryToWatch);
					}
				}
			}
		}
	}
#endif // WITH_EDITOR
}

FShaderCompileThreadRunnableBase* FShaderCompilingManager::LaunchShaderCompilingThread(TUniquePtr<FShaderCompileThreadRunnableBase>&& NewShaderCompilingThread, bool bDelayThreadExecution)
{
	// Take ownership of new shader compiling thread
	FShaderCompileThreadRunnableBase* ThreadRef = NewShaderCompilingThread.Get();
	Threads.Add(MoveTemp(NewShaderCompilingThread));

	if (!bDelayThreadExecution)
	{
		ThreadRef->StartThread();
	}

	return ThreadRef;
}

FShaderCompileThreadRunnableBase* FShaderCompilingManager::LaunchRemoteShaderCompilingThread(bool bDelayThreadExecution)
{
	// Check if there already is a distributed compile thread
	if (FShaderCompileThreadRunnableBase* DistributedCompileThread = FindShaderCompilingThread(EShaderCompilerWorkerType::Distributed))
	{
		return DistributedCompileThread;
	}

	// Check if remote compiling is allowed and find distributed controller
	const bool bCanUseRemoteCompiling = bAllowCompilingThroughWorkers && ShaderCompiler::IsRemoteCompilingAllowed() && AllTargetPlatformSupportsRemoteShaderCompiling();
	BuildDistributionController = bCanUseRemoteCompiling ? FindRemoteCompilerController() : nullptr;

	if (BuildDistributionController)
	{
		// Initialize distributed controller with worker limits if the controller also supports local workers
		BuildDistributionController->SetMaxLocalWorkers(GetNumLocalWorkers());

		// Allocate distributed shader compiling thread
		UE_LOGF(LogShaderCompilers, Display, "Using %ls for shader compilation", *BuildDistributionController->GetName());
		TUniquePtr<FShaderCompileThreadRunnableBase> RemoteCompileThread = MakeUnique<FShaderCompileDistributedThreadRunnable_Interface>(this, *BuildDistributionController);

		const bool bExclusiveRemoteShaderCompiling = FParse::Param(FCommandLine::Get(), TEXT("ExclusiveRemoteShaderCompiling"));
		const bool bDistributedControllerSupportsLocalWorkers = BuildDistributionController->SupportsLocalWorkers();

		bUseOnlyDistributedCompilationThread = bDistributedControllerSupportsLocalWorkers || bExclusiveRemoteShaderCompiling;
		return LaunchShaderCompilingThread(MoveTemp(RemoteCompileThread), bDelayThreadExecution);
	}

	// Print information why no distributed shader compiler controller was launched, since local compilation is usually only the fallback
	if (bCanUseRemoteCompiling)
	{
		UE_LOGF(LogShaderCompilers, Display, "No distributed shader compiler controller found");
	}
	else if (!AllTargetPlatformSupportsRemoteShaderCompiling())
	{
		UE_LOGF(LogShaderCompilers, Display, "Distributed shader compilation is not supported for all target platforms");
	}
	else
	{
		UE_LOGF(LogShaderCompilers, Display, "Distributed shader compilation is disabled");
	}

	return nullptr;
}

FShaderCompileThreadRunnableBase* FShaderCompilingManager::LaunchLocalShaderCompilingThread(bool bDelayThreadExecution)
{
	// Check if there already is a local compile thread
	if (FShaderCompileThreadRunnableBase* LocalCompileThread = FindShaderCompilingThread(EShaderCompilerWorkerType::LocalThread))
	{
		return LocalCompileThread;
	}

	// Allocate local shader compiling thread
	UE_LOGF(LogShaderCompilers, Display, "Using %d local workers for shader compilation", NumShaderCompilingThreads);
	TUniquePtr<FShaderCompileThreadRunnableBase> LocalThread = MakeUnique<FShaderCompileThreadRunnable>(this);

	if (GIsBuildMachine)
	{
		int32 MinSCWsToSpawnBeforeWarning = 8; // optional, default to 8
		GConfig->GetInt(TEXT("DevOptions.Shaders"), TEXT("MinSCWsToSpawnBeforeWarning"), MinSCWsToSpawnBeforeWarning, GEngineIni);
		if (NumShaderCompilingThreads < static_cast<uint32>(MinSCWsToSpawnBeforeWarning))
		{
			UE_LOGF(LogShaderCompilers, Warning, "Only %d SCWs will be spawned, which will result in longer shader compile times.", NumShaderCompilingThreads);
		}
	}

	if (FShaderCompileThreadRunnableBase* Thread = LaunchShaderCompilingThread(MoveTemp(LocalThread), bDelayThreadExecution))
	{
		bUseOnlyDistributedCompilationThread = false;
		return Thread;
	}

	return nullptr;
}

FShaderCompileThreadRunnableBase* FShaderCompilingManager::FindShaderCompilingThread(EShaderCompilerWorkerType InWorkerType)
{
	for (const auto& Thread : Threads)
	{
		if (Thread->GetWorkerType() == InWorkerType)
		{
			return Thread.Get();
		}
	}
	return nullptr;
}

void FShaderCompilingManager::OnShaderCompilingThreadStarted(FShaderCompileThreadRunnableBase* /*Thread*/)
{
	// Update assigned priority ranges for all threads
	AssignShaderCompilingThreadJobPriorities();
}

void FShaderCompilingManager::OnShaderCompilingThreadStopped(FShaderCompileThreadRunnableBase* Thread)
{
	// If the manager is shutting down, ignore this notification
	if (bIsManagerShuttingDown)
	{
		return;
	}

	// Remove shader compiling thread from list of active threads
	for (auto Iter = Threads.CreateIterator(); Iter; ++Iter)
	{
		if (Iter->Get() == Thread)
		{
			// Move thread ownership to shutdown task. Its memory needs to be kept alive until the thread has finished.
			// Let the task system handle this deferred deletion, since this notification callback is called within the shader compiling threads and waiting for its compiltion in this function would deadlock.
			checkf((*Iter)->bForceFinish, TEXT("%s is signaled to have stopped but is not put into the right state"), (*Iter)->GetThreadName());

			const FString TaskName = FString::Printf(TEXT("ShutdownTask-%s"), Thread->GetThreadName());
			UE::Tasks::Launch(
				*TaskName,
				[Thread = MoveTemp(*Iter)](){
					Thread->WaitForCompletion();
				}
			);

			Iter.RemoveCurrent();
			break;
		}
	}

	// Update assigned priority ranges for the remaining threads
	AssignShaderCompilingThreadJobPriorities();
}

void FShaderCompilingManager::AssignShaderCompilingThreadJobPriorities()
{
	// If there is more than one thread for shader compilation, re-arrange distribution of job priorities, to avoid all threads picking up the same type of jobs
	if (Threads.Num() >= 2)
	{
		// Only force-local jobs are guaranteed to stay on the local machine. Going wide with High priority jobs is important for the startup times,
		// since special materials use High priority. Possibly the partition by priority is too rigid in general.
		for (const auto& Thread : Threads)
		{
			FShaderCompileThreadRunnableBase* CompileThread = Thread.Get();
			switch (Thread->GetWorkerType())
			{
			case EShaderCompilerWorkerType::None:
				checkNoEntry();
				break;
			case EShaderCompilerWorkerType::LocalThread:
				CompileThread->SetPriorityRange(EShaderCompileJobPriority::Normal, EShaderCompileJobPriority::ForceLocal);
				break;
			case EShaderCompilerWorkerType::Distributed:
				CompileThread->SetPriorityRange(EShaderCompileJobPriority::Low, EShaderCompileJobPriority::ExtraHigh);
				break;
			}
		}
	}
	else if (Threads.Num() == 1)
	{
		// A single shader compiling thread must take care of all job priorities
		Threads[0]->SetPriorityRange(EShaderCompileJobPriority::Low, EShaderCompileJobPriority::ForceLocal);
	}
}

FShaderCompilingManager::~FShaderCompilingManager()
{
	bIsManagerShuttingDown = true;

	// we never initialized, so nothing to do
	if (!AllowShaderCompiling())
	{
		return;
	}

	for (const auto& Thread : Threads)
	{
		Thread->Stop();
		Thread->WaitForCompletion();
	}

	FCoreDelegates::GetOutOfMemoryDelegate().Remove(OutOfMemoryDelegateHandle);

#if WITH_EDITOR
	const bool bAllowShaderRecompileOnSave = CVarRecompileShadersOnSave.GetValueOnAnyThread();
	if (bAllowShaderRecompileOnSave)
	{
		if (IDirectoryWatcher* DirectoryWatcher = FModuleManager::LoadModuleChecked<FDirectoryWatcherModule>(TEXT("DirectoryWatcher")).Get())
		{
			for (const auto& It : DirectoryWatcherHandles)
			{
				DirectoryWatcher->UnregisterDirectoryChangedCallback_Handle(It.Key, It.Value);
			}
		}
	}
#endif // WITH_EDITOR

	FAssetCompilingManager::Get().UnregisterManager(this);
}

void FShaderCompilingManager::CalculateNumberOfCompilingThreads(int32 NumberOfCores, int32 NumberOfCoresIncludingHyperthreads)
{
	const int32 NumVirtualCores = NumberOfCoresIncludingHyperthreads;

	int32 NumUnusedShaderCompilingThreads;
	verify(GConfig->GetInt(TEXT("DevOptions.Shaders"), TEXT("NumUnusedShaderCompilingThreads"), NumUnusedShaderCompilingThreads, GEngineIni));

	int32 NumUnusedShaderCompilingThreadsDuringGame;
	verify(GConfig->GetInt(TEXT("DevOptions.Shaders"), TEXT("NumUnusedShaderCompilingThreadsDuringGame"), NumUnusedShaderCompilingThreadsDuringGame, GEngineIni));

	int32 ShaderCompilerCoreCountThreshold;
	verify(GConfig->GetInt(TEXT("DevOptions.Shaders"), TEXT("ShaderCompilerCoreCountThreshold"), ShaderCompilerCoreCountThreshold, GEngineIni));

	bool bForceUseSCWMemoryPressureLimits = false;
	GConfig->GetBool(TEXT("DevOptions.Shaders"), TEXT("bForceUseSCWMemoryPressureLimits"), bForceUseSCWMemoryPressureLimits, GEngineIni);

	// Don't reserve threads based on a percentage if we are in a commandlet or on a low core machine.
	// In these scenarios we should try to use as many threads as possible.
	if (!IsRunningCommandlet() && !GIsBuildMachine && NumVirtualCores > ShaderCompilerCoreCountThreshold)
	{
		// Reserve a percentage of the threads for general background work.
		float PercentageUnusedShaderCompilingThreads;
		verify(GConfig->GetFloat(TEXT("DevOptions.Shaders"), TEXT("PercentageUnusedShaderCompilingThreads"), PercentageUnusedShaderCompilingThreads, GEngineIni));

		// ensure we get a valid multiplier.
		PercentageUnusedShaderCompilingThreads = FMath::Clamp(PercentageUnusedShaderCompilingThreads, 0.0f, 100.0f) / 100.0f;

		NumUnusedShaderCompilingThreads = FMath::CeilToInt(NumVirtualCores * PercentageUnusedShaderCompilingThreads);
		NumUnusedShaderCompilingThreadsDuringGame = NumUnusedShaderCompilingThreads;
	}

	// Use all the cores on the build machines.
	if (GForceAllCoresForShaderCompiling != 0)
	{
		NumUnusedShaderCompilingThreads = 0;
	}

	NumShaderCompilingThreads = (bAllowCompilingThroughWorkers && NumVirtualCores > NumUnusedShaderCompilingThreads) ? (NumVirtualCores - NumUnusedShaderCompilingThreads) : 1;

	// Make sure there's at least one worker allowed to be active when compiling during the game
	NumShaderCompilingThreadsDuringGame = (bAllowCompilingThroughWorkers && NumVirtualCores > NumUnusedShaderCompilingThreadsDuringGame) ? (NumVirtualCores - NumUnusedShaderCompilingThreadsDuringGame) : 1;

	// On machines with few cores, each core will have a massive impact on compile time, so we prioritize compile latency over editor performance during the build
	if (NumVirtualCores <= 4)
	{
		NumShaderCompilingThreads = NumVirtualCores - 1;
		NumShaderCompilingThreadsDuringGame = NumVirtualCores - 1;
	}
#if PLATFORM_DESKTOP
	else if (GIsBuildMachine || bForceUseSCWMemoryPressureLimits)
	{
		// Cooker ends up running OOM so use a simple heuristic based on some INI values
		float CookerMemoryUsedInGB = 0.0f;
		float MemoryToLeaveForTheOSInGB = 0.0f;
		float MemoryUsedPerSCWProcessInGB = 0.0f;
		bool bFoundEntries = true;
		bFoundEntries = bFoundEntries && GConfig->GetFloat(TEXT("DevOptions.Shaders"), TEXT("CookerMemoryUsedInGB"), CookerMemoryUsedInGB, GEngineIni);
		bFoundEntries = bFoundEntries && GConfig->GetFloat(TEXT("DevOptions.Shaders"), TEXT("MemoryToLeaveForTheOSInGB"), MemoryToLeaveForTheOSInGB, GEngineIni);
		bFoundEntries = bFoundEntries && GConfig->GetFloat(TEXT("DevOptions.Shaders"), TEXT("MemoryUsedPerSCWProcessInGB"), MemoryUsedPerSCWProcessInGB, GEngineIni);
		if (bFoundEntries)
		{
			uint32 PhysicalGBRam = FPlatformMemory::GetPhysicalGBRam();
			float AvailableMemInGB = (float)PhysicalGBRam - CookerMemoryUsedInGB;
			if (AvailableMemInGB > 0.0f)
			{
				if (AvailableMemInGB > MemoryToLeaveForTheOSInGB)
				{
					AvailableMemInGB -= MemoryToLeaveForTheOSInGB;
				}
				else
				{
					UE_LOGF(LogShaderCompilers, Display, "Machine has %d GBs of RAM, cooker might take %f GBs, but not enough memory left for the OS! (Requested %f GBs for the OS)", PhysicalGBRam, CookerMemoryUsedInGB, MemoryToLeaveForTheOSInGB);
				}
			}
			else
			{
				UE_LOGF(LogShaderCompilers, Display, "Machine has %d GBs of RAM, but cooker might take %f GBs!", PhysicalGBRam, CookerMemoryUsedInGB);
			}
			if (MemoryUsedPerSCWProcessInGB > 0.0f)
			{
				float NumSCWs = AvailableMemInGB / MemoryUsedPerSCWProcessInGB;
				NumShaderCompilingThreads = FMath::RoundToInt(NumSCWs);

				bool bUseVirtualCores = true;
				GConfig->GetBool(TEXT("DevOptions.Shaders"), TEXT("bUseVirtualCores"), bUseVirtualCores, GEngineIni);
				uint32 MaxNumCoresToUse = bUseVirtualCores ? NumVirtualCores : NumberOfCores;
				NumShaderCompilingThreads = FMath::Clamp<uint32>(NumShaderCompilingThreads, 1, MaxNumCoresToUse - 1);
				NumShaderCompilingThreadsDuringGame = FMath::Min<int32>(NumShaderCompilingThreads, NumShaderCompilingThreadsDuringGame);
			}
		}
		else if (bForceUseSCWMemoryPressureLimits)
		{
			UE_LOGF(LogShaderCompilers, Warning, "bForceUseSCWMemoryPressureLimits was set but missing one or more prerequisite setting(s): CookerMemoryUsedInGB, MemoryToLeaveForTheOSInGB, MemoryUsedPerSCWProcessInGB.  Ignoring bForceUseSCWMemoryPressureLimits");
		}

		if (GIsBuildMachine)
		{
			// force crashes on hung shader maps on build machines, to prevent builds running for days
			GCrashOnHungShaderMaps = 1;
		}
	}
#endif

	NumShaderCompilingThreads = FMath::Max<int32>(1, NumShaderCompilingThreads);
	NumShaderCompilingThreadsDuringGame = FMath::Max<int32>(1, NumShaderCompilingThreadsDuringGame);

	NumShaderCompilingThreadsDuringGame = FMath::Min<int32>(NumShaderCompilingThreadsDuringGame, NumShaderCompilingThreads);
}

void FShaderCompilingManager::OnMachineResourcesChanged(int32 NumberOfCores, int32 NumberOfCoresIncludingHyperthreads)
{
	CalculateNumberOfCompilingThreads(NumberOfCores, NumberOfCoresIncludingHyperthreads);

	if (BuildDistributionController)
	{
		BuildDistributionController->SetMaxLocalWorkers(GetNumLocalWorkers());
	}

	for (const auto& Thread : Threads)
	{
		Thread->OnMachineResourcesChanged();
	}
}

void FShaderCompilingManager::OnDistributedShaderCompilingChanged()
{
	if (BuildDistributionController)
	{
		// Only update conditions if local shader compiling thread was initially disabled but is now required
		const bool bDistributedControllerSupportsLocalWorkers = BuildDistributionController->SupportsLocalWorkers();
		if (bUseOnlyDistributedCompilationThread && !bDistributedControllerSupportsLocalWorkers)
		{
			// Launch local shader compiling thread, since the distributed controller no longer supports local workers
			if (LaunchLocalShaderCompilingThread() != nullptr)
			{
				bUseOnlyDistributedCompilationThread = false;
			}
		}
	}
}

FName FShaderCompilingManager::GetStaticAssetTypeName()
{
	return TEXT("UE-Shader");
}

FName FShaderCompilingManager::GetAssetTypeName() const
{
	return GetStaticAssetTypeName();
}

FTextFormat FShaderCompilingManager::GetAssetNameFormat() const
{
	return LOCTEXT("ShaderNameFormat", "{0}|plural(one=Shader,other=Shaders)");
}

TArrayView<FName> FShaderCompilingManager::GetDependentTypeNames() const
{
#if WITH_EDITOR
	static FName DependentTypeNames[] = 
	{
		// Texture can require materials to be updated,
		// they should be processed first to avoid unecessary material updates.
		FTextureCompilingManager::GetStaticAssetTypeName() 
	};
	return TArrayView<FName>(DependentTypeNames);
#else
	return TArrayView<FName>();
#endif	
}

int32 FShaderCompilingManager::GetNumRemainingAssets() const
{
	// Currently, jobs are difficult to track but the purpose of the GetNumRemainingAssets function is to never return 0
	// if there are still shaders that have not had their primitives updated on the render thread.
	// So we track jobs first and when everything is finished compiling but are still lying around in other structures
	// waiting to be further processed, we show those numbers and ultimately we always return 1 unless IsCompiling() is false.
	return FMath::Max3(GetNumRemainingJobs(), ShaderMapJobs.Num() + PendingFinalizeShaderMaps.Num(), IsCompiling() ? 1 : 0);
}

void FShaderCompilingManager::ProcessAsyncTasks(bool bLimitExecutionTime)
{
	ProcessAsyncResults(bLimitExecutionTime, false);
}

void FShaderCompilingManager::ProcessAsyncTasks(const AssetCompilation::FProcessAsyncTaskParams& Params)
{
	// Shader compilations are not required for PIE to begin.
	if (Params.bPlayInEditorAssetsOnly)
	{
		return;
	}

	ProcessAsyncResults(Params.bLimitExecutionTime, false);
}

int32 FShaderCompilingManager::GetNumPendingJobs() const
{
	return AllJobs.GetNumPendingJobs();
}

int32 FShaderCompilingManager::GetNumOutstandingJobs() const
{
	return AllJobs.GetNumOutstandingJobs();
}

FShaderCompilingManager::EDumpShaderDebugInfo FShaderCompilingManager::GetDumpShaderDebugInfo() const
{
	if (GDumpShaderDebugInfo < EDumpShaderDebugInfo::Never || GDumpShaderDebugInfo > EDumpShaderDebugInfo::OnErrorOrWarning)
	{
		return EDumpShaderDebugInfo::Never;
	}

	return static_cast<FShaderCompilingManager::EDumpShaderDebugInfo>(GDumpShaderDebugInfo);
}

EShaderDebugInfoFlags FShaderCompilingManager::GetDumpShaderDebugInfoFlags() const
{
	EShaderDebugInfoFlags Flags = EShaderDebugInfoFlags::Default;
	
	if (CVarDebugDumpJobInputHashes.GetValueOnAnyThread())
	{
		Flags |= EShaderDebugInfoFlags::InputHash;
	}

	if (CVarDebugDumpJobDiagnostics.GetValueOnAnyThread())
	{
		Flags |= EShaderDebugInfoFlags::Diagnostics;
	}

	if (CVarDebugDumpShaderCode.GetValueOnAnyThread())
	{
		Flags |= EShaderDebugInfoFlags::ShaderCodeBinary;
	}

	if (CVarDebugDumpShaderCodePlatformHashes.GetValueOnAnyThread())
	{
		Flags |= EShaderDebugInfoFlags::ShaderCodePlatformHashes;
	}

	if (CVarDebugDumpDetailedShaderSource.GetValueOnAnyThread())
	{
		Flags |= EShaderDebugInfoFlags::DetailedSource;
	}

	return Flags;
}

FString FShaderCompilingManager::CreateShaderDebugInfoPath(const FShaderCompilerInput& ShaderCompilerInput) const
{
	return ShaderCompilerInput.GetOrCreateShaderDebugInfoPath();
}

bool FShaderCompilingManager::ShouldRecompileToDumpShaderDebugInfo(const FShaderCompileJob& Job) const
{
	return ShouldRecompileToDumpShaderDebugInfo(Job.Input, Job.Output, Job.bSucceeded);
}

bool FShaderCompilingManager::ShouldRecompileToDumpShaderDebugInfo(const FShaderCompilerInput& Input, const FShaderCompilerOutput& Output, bool bSucceeded) const
{
	if (Input.DumpDebugInfoPath.IsEmpty())
	{
		const EDumpShaderDebugInfo DumpShaderDebugInfo = GetDumpShaderDebugInfo();
		const bool bErrors = !bSucceeded;
		const bool bWarnings = Output.Errors.Num() > 0;

		bool bShouldDump = true;
		if (GIsBuildMachine)
		{
			// Build machines dump these as build artifacts and they should only upload so many due to size constraints.
			bShouldDump = (NumDumpedShaderSources < GMaxNumDumpedShaderSources);
		}

		if (DumpShaderDebugInfo == EDumpShaderDebugInfo::OnError)
		{
			return bShouldDump && (bErrors);
		}
		else if (DumpShaderDebugInfo == EDumpShaderDebugInfo::OnErrorOrWarning || Output.bForceDumpDebugInfo)
		{
			return bShouldDump && (bErrors || bWarnings);
		}
	}

	return false;
}

void FShaderCompilingManager::ReleaseJob(FShaderCommonCompileJobPtr& Job)
{
	ReleaseJob(Job.GetReference());
	Job.SafeRelease();
}

void FShaderCompilingManager::ReleaseJob(FShaderCommonCompileJob* Job)
{
	Job->PendingShaderMap.SafeRelease();
	Job->UpdateStatus(EShaderCompileJobStatus::Released);
	AllJobs.RemoveJob(Job);
}

void FShaderCompilingManager::SubmitJobs(TArray<FShaderCommonCompileJobPtr>& NewJobs, const FString MaterialBasePath, const FString PermutationString)
{
	LLM_SCOPE_BYTAG(ShaderCompiler);

	// make sure no compiling can start if not allowed
	if (!AllowShaderCompiling())
	{
		return;
	}
	
	TRACE_CPUPROFILER_EVENT_SCOPE(FShaderCompilingManager::SubmitJobs);
	check(!FPlatformProperties::RequiresCookedData());

	if (NewJobs.Num() == 0)
	{
		return;
	}

	check(GShaderCompilerStats);
	if (FShaderCompileJob* SingleJob = NewJobs[0]->GetSingleShaderJob()) //assume that all jobs are for the same platform
	{
		GShaderCompilerStats->RegisterCompiledShaders(NewJobs.Num(), SingleJob->Input.Target.GetPlatform(), MaterialBasePath, PermutationString);
	}
	else
	{
		GShaderCompilerStats->RegisterCompiledShaders(NewJobs.Num(), SP_NumPlatforms, MaterialBasePath, PermutationString);
	}

	{
		FScopeLock Lock(&CompileQueueSection);
		for (auto& Job : NewJobs)
		{
			FPendingShaderMapCompileResultsPtr& PendingShaderMap = ShaderMapJobs.FindOrAdd(Job->Id);
			if (!PendingShaderMap)
			{
				PendingShaderMap = new FPendingShaderMapCompileResults();
			}
			PendingShaderMap->NumPendingJobs.Increment();
			Job->PendingShaderMap = PendingShaderMap;
		}

		// in the case of submitting jobs from worker threads we need to be sure that the lock extends to
		// include AllJobs.SubmitJobs().  This will increase contention for the lock, but this will let us
		// prototype getting shader translation and preprocessing being done on worker threads.
		if (IsInGameThread())
		{
			Lock.Unlock();
		}

		AllJobs.SubmitJobs(NewJobs);
	}

	UpdateNumRemainingAssets();
}

void FShaderCompilingManager::TransferJobsToLocalWorker(TArray<FShaderCommonCompileJobPtr>& JobsToTransfer)
{
	// Make sure we have a local shader compiling thread
	if (FindShaderCompilingThread(EShaderCompilerWorkerType::LocalThread) == nullptr)
	{
		LaunchLocalShaderCompilingThread();
	}

	FScopeLock Lock(&CompileQueueSection);
	AllJobs.TransferJobsToLocalWorker(JobsToTransfer);
}

bool FShaderCompilingManager::IsCompilingShaderMap(uint32 Id)
{
	if (Id != 0u)
	{
		FScopeLock Lock(&CompileQueueSection);
		FPendingShaderMapCompileResultsPtr* PendingShaderMapPtr = ShaderMapJobs.Find(Id);
		if (PendingShaderMapPtr)
		{
			return true;
		}

		FShaderMapFinalizeResults* FinalizedShaderMapPtr = PendingFinalizeShaderMaps.Find(Id);
		if (FinalizedShaderMapPtr)
		{
			return true;
		}
	}
	return false;
}

FShaderCompileJob* FShaderCompilingManager::PrepareShaderCompileJob(uint32 Id, const FShaderCompileJobKey& Key, EShaderCompileJobPriority Priority)
{
	// don't allow any jobs if not allowed
	if (!AllowShaderCompiling())
	{
		return nullptr;
	}

#if WITH_EDITOR
	// Check if shader type overrides job priority
	check(Key.ShaderType != nullptr);
	const EShaderCompileJobPriority OverrideJobPriority = Key.ShaderType->GetOverrideJobPriority();
	Priority = (OverrideJobPriority < EShaderCompileJobPriority::Num ? FMath::Max(OverrideJobPriority, Priority) : Priority);
#endif

	return AllJobs.PrepareJob(Id, Key, Priority);
}

FShaderPipelineCompileJob* FShaderCompilingManager::PreparePipelineCompileJob(uint32 Id, const FShaderPipelineCompileJobKey& Key, EShaderCompileJobPriority Priority)
{
	// don't allow any jobs if not allowed
	if (!AllowShaderCompiling())
	{
		return nullptr;
	}

#if WITH_EDITOR
	// Check if shader types in pipeline override job priority and pick highest one
	check(Key.ShaderPipeline != nullptr);
	for (const FShaderType* ShaderType : Key.ShaderPipeline->GetStages())
	{
		const EShaderCompileJobPriority OverrideJobPriority = ShaderType->GetOverrideJobPriority();
		Priority = (OverrideJobPriority < EShaderCompileJobPriority::Num ? FMath::Max(OverrideJobPriority, Priority) : Priority);
	}
#endif

	return AllJobs.PrepareJob(Id, Key, Priority);
}

void FShaderCompilingManager::ProcessFinishedJob(FShaderCommonCompileJob* FinishedJob, EShaderCompileJobStatus Status)
{
	bool bIsPipelineJob = FinishedJob->Type == EShaderCompileJobType::Pipeline;
	FinishedJob->ForEachSingleShaderJob([this](FShaderCompileJob& SingleJob)
		{
			// Log if requested or if there was an exceptionally slow batch, to see the offender easily
			if (bLogJobCompletionTimes || SingleJob.Output.CompileTime > (double)CVarShaderCompilerLogSlowJobThreshold.GetValueOnAnyThread())
			{
				TStringBuilder<256> JobName;
				if (SingleJob.Input.DumpDebugInfoEnabled())
				{
					JobName << SingleJob.Input.DumpDebugInfoPath;
				}
				else
				{
					JobName << SingleJob.Key.ShaderType->GetName();
					JobName.Appendf(TEXT("(permutation %d, format %s)"), SingleJob.Key.PermutationId, *SingleJob.Input.ShaderFormat.ToString());
				}
				UE_LOGF(LogShaderCompilers, Display, "Job %ls compile time exceeded threshold (%.3fs)", JobName.ToString(), SingleJob.Output.CompileTime);
			}
		});

	AllJobs.ProcessFinishedJob(FinishedJob, Status);
}

/** Launches the worker, returns the launched process handle. */
FProcHandle FShaderCompilingManager::LaunchWorker(const FString& WorkingDirectory, uint32 InParentProcessId, uint32 ThreadId, const FString& WorkerInputFile, const FString& WorkerOutputFile, uint32* OutWorkerProcessId)
{
	// don't allow any jobs if not allowed
	if (!AllowShaderCompiling())
	{
		return FProcHandle();
	}

	// Setup the parameters that the worker application needs
	// Surround the working directory with double quotes because it may contain a space 
	// WorkingDirectory ends with a '\', so we have to insert another to meet the Windows commandline parsing rules 
	// http://msdn.microsoft.com/en-us/library/17w5ykft.aspx 
	// Use IFileManager to do path conversion to properly handle sandbox paths (outside of standard paths in particular).
	FString WorkerAbsoluteDirectory = IFileManager::Get().ConvertToAbsolutePathForExternalAppForWrite(*WorkingDirectory);
	FPaths::NormalizeDirectoryName(WorkerAbsoluteDirectory);
	TStringBuilder<1024> WorkerParameters;
	WorkerParameters << TEXT("\"") << WorkerAbsoluteDirectory << TEXT("/\" ") << InParentProcessId << TEXT(" ") << ThreadId << " " << WorkerInputFile << " " << WorkerOutputFile << TEXT(" -communicatethroughfile ");

	if (GIsBuildMachine)
	{
		WorkerParameters.Appendf(TEXT(" -TimeToLive=%f -buildmachine"), GBuildWorkerTimeToLive);
	}
	else
	{
		WorkerParameters.Appendf(TEXT(" -TimeToLive=%f"), GRegularWorkerTimeToLive);
	}
	if (PLATFORM_LINUX)
	{
		// suppress log generation as much as possible
		WorkerParameters << TEXT(" -logcmds=\"Global None\"");

		if (UE_BUILD_DEBUG)
		{
			// when running a debug build under Linux, make SCW crash with core for easier debugging
			WorkerParameters << TEXT(" -core");
		}
	}
	FCommandLine::BuildSubprocessCommandLine(ECommandLineArgumentFlags::ProgramContext, false /*bOnlyInherited*/, WorkerParameters);

#if USE_SHADER_COMPILER_WORKER_TRACE
	// When doing utrace functionality we can't run with -nothreading, since it won't create the utrace thread to send events.
	WorkerParameters << TEXT(" -trace=default");
#else
	WorkerParameters << TEXT(" -nothreading");
#endif // USE_SHADER_COMPILER_WORKER_TRACE

	if (GDebugDumpWorkerCrashLog)
	{
		WorkerParameters << TEXT(" -LogToMemory -DumpLogOnExitCrashOnly");

		const FString WorkerLogFilename = FString::Printf(TEXT("ShaderCompileWorker-%d.log"), ThreadId);
		if (!WorkerCrashLogBaseDirectory.IsEmpty())
		{
			WorkerParameters.Appendf(TEXT(" -AbsLog=\"%s\""), *FPaths::Combine(WorkerCrashLogBaseDirectory, WorkerLogFilename));
		}
		else
		{
			WorkerParameters.Appendf(TEXT(" -Log=%s"), *WorkerLogFilename);
		}
	}

	//Inherit the base directory from the engine process
	FString BaseDirOverride;
	if (FParse::Value(FCommandLine::Get(), TEXT("basedir="), BaseDirOverride))
	{
		WorkerParameters.Appendf(TEXT(" -basedir=\"%s\""), *BaseDirOverride);
	}

	FString BaseFromWorkingDirOverride;
	if (FParse::Value(FCommandLine::Get(), TEXT("BaseFromWorkingDir="), BaseFromWorkingDirOverride))
	{
		WorkerParameters.Appendf(TEXT(" -BaseFromWorkingDir=\"%s\""), *BaseFromWorkingDirOverride);
	}

	if (DEBUG_SHADERCOMPILEWORKER)
	{
		// Note: Set breakpoint here and launch the ShaderCompileWorker with WorkerParameters a cmd-line
		const TCHAR* WorkerParametersText = *WorkerParameters;
		FPlatformMisc::LowLevelOutputDebugStringf(TEXT("Launching shader compile worker w/ WorkerParameters\n\t%s\n"), WorkerParametersText);
		FProcHandle DummyHandle;
		return DummyHandle;
	}
	else
	{
#if UE_BUILD_DEBUG && PLATFORM_LINUX
		FPlatformMisc::LowLevelOutputDebugStringf(TEXT("Launching shader compile worker:\n\t%s\n"), *WorkerParameters);
#endif

		// Launch the worker process
		int32 PriorityModifier = -1; // below normal
		GConfig->GetInt(TEXT("DevOptions.Shaders"), TEXT("WorkerProcessPriority"), PriorityModifier, GEngineIni);

		// Disambiguate between SCW.exe missing vs other errors.
		static bool bFirstLaunch = true;
		uint32 WorkerId = 0;
		FProcHandle WorkerHandle = FPlatformProcess::CreateProc(*ShaderCompileWorkerName, *WorkerParameters, true, false, false, &WorkerId, PriorityModifier, nullptr, nullptr);
		if (WorkerHandle.IsValid())
		{
			if (OutWorkerProcessId)
			{
				*OutWorkerProcessId = WorkerId;
			}
			// Process launched at least once successfully
			bFirstLaunch = false;
		}
		else
		{
			// If this doesn't error, the app will hang waiting for jobs that can never be completed
			if (bFirstLaunch)
			{
				// When using source builds users are likely to make a mistake of not building SCW (e.g. in particular on Linux, even though default makefile target builds it).
				// Make the engine exit gracefully with a helpful message instead of a crash.
				static bool bShowedMessageBox = false;
				if (!bShowedMessageBox && !IsRunningCommandlet() && !FApp::IsUnattended())
				{
					bShowedMessageBox = true;
					FText ErrorMessage = FText::Format(LOCTEXT("LaunchingShaderCompileWorkerFailed", "Unable to launch {0} - make sure you built ShaderCompileWorker."), FText::FromString(ShaderCompileWorkerName));
					FPlatformMisc::MessageBoxExt(EAppMsgType::Ok, *ErrorMessage.ToString(),
												 *LOCTEXT("LaunchingShaderCompileWorkerFailedTitle", "Unable to launch ShaderCompileWorker.").ToString());
				}
				UE_LOGF(LogShaderCompilers, Error, "Couldn't launch %ls! Make sure you build ShaderCompileWorker.", *ShaderCompileWorkerName);
				// duplicate to printf() since threaded logs may not be always flushed
				FPlatformMisc::LowLevelOutputDebugStringf(TEXT("Couldn't launch %s! Make sure you build ShaderCompileWorker.\n"), *ShaderCompileWorkerName);
				FPlatformMisc::RequestExitWithStatus(true, 1);
			}
			else
			{
				UE_LOGF(LogShaderCompilers, Fatal, "Couldn't launch %ls!", *ShaderCompileWorkerName);
			}
		}

		return WorkerHandle;
	}
}

void FShaderCompilingManager::AddCompiledResults(TMap<int32, FShaderMapFinalizeResults>& CompiledShaderMaps, int32 ShaderMapIdx, const FShaderMapFinalizeResults& Results)
{
	// merge with the previous unprocessed jobs, if any
	if (FShaderMapCompileResults const* PrevResults = CompiledShaderMaps.Find(ShaderMapIdx))
	{
		FShaderMapFinalizeResults NewResults(Results);

		NewResults.bAllJobsSucceeded = NewResults.bAllJobsSucceeded && PrevResults->bAllJobsSucceeded;
		NewResults.bSkipResultProcessing = NewResults.bSkipResultProcessing || PrevResults->bSkipResultProcessing;
		NewResults.TimeStarted = FMath::Min(NewResults.TimeStarted, PrevResults->TimeStarted);
		NewResults.bIsHung = NewResults.bIsHung || PrevResults->bIsHung;
		NewResults.FinishedJobs.Append(PrevResults->FinishedJobs);

		CompiledShaderMaps.Add(ShaderMapIdx, NewResults);
	}
	else
	{
		CompiledShaderMaps.Add(ShaderMapIdx, Results);
	}
}

/** Flushes all pending jobs for the given shader maps. */
void FShaderCompilingManager::BlockOnShaderMapCompletion(const TArray<int32>& ShaderMapIdsToFinishCompiling, TMap<int32, FShaderMapFinalizeResults>& CompiledShaderMaps)
{
	// never block if no compiling, just in case
	if (!AllowShaderCompiling())
	{
		return;
	}

	TRACE_CPUPROFILER_EVENT_SCOPE(FShaderCompilingManager::BlockOnShaderMapCompletion);

	COOK_STAT(FScopedDurationAtomicTimer BlockingTimer(ShaderCompilerCookStats::BlockingTimeSec));
	if (bAllowAsynchronousShaderCompiling)
	{
		// Calculate how many shader jobs there are total to provide the slow task with the correct amount of work.
		int NumJobs = 0;
		{
			FScopeLock Lock(&CompileQueueSection);
			for (int32 ShaderMapIndex = 0; ShaderMapIndex < ShaderMapIdsToFinishCompiling.Num(); ShaderMapIndex++)
			{
				FPendingShaderMapCompileResultsPtr* ResultsPtr = ShaderMapJobs.Find(ShaderMapIdsToFinishCompiling[ShaderMapIndex]);
				if (ResultsPtr)
				{
					FShaderMapCompileResults* Results = *ResultsPtr;
					NumJobs += Results->NumPendingJobs.GetValue();
				}
			}
		}

		FScopedSlowTask SlowTask(NumJobs, FText::Format(LOCTEXT("BlockOnShaderMapCompletion", "Compiling Shaders ({0})"), NumJobs), GIsEditor && !IsRunningCommandlet() && UE::GetPlayInEditorID() == INDEX_NONE);
		if (NumJobs > 0)
		{
			SlowTask.MakeDialogDelayed(1.0f);
		}

		int32 NumPendingJobs = 0;
		// Keep track of previous number of pending jobs so we can update the slow task with the amount of work done.
		int32 NumPreviousPendingJobs = NumJobs;
		int32 LogCounter = 0;
		do 
		{
			NumPendingJobs = 0;
			{
				// Lock CompileQueueSection so we can access the input and output queues
				FScopeLock Lock(&CompileQueueSection);

				for (int32 ShaderMapIndex = 0; ShaderMapIndex < ShaderMapIdsToFinishCompiling.Num(); ShaderMapIndex++)
				{
					FPendingShaderMapCompileResultsPtr* ResultsPtr = ShaderMapJobs.Find(ShaderMapIdsToFinishCompiling[ShaderMapIndex]);
					if (ResultsPtr)
					{
						FShaderMapCompileResults* Results = *ResultsPtr;

						if (Results->NumPendingJobs.GetValue() == 0)
						{
							if (Results->FinishedJobs.Num() > 0)
							{
								AddCompiledResults(CompiledShaderMaps, ShaderMapIdsToFinishCompiling[ShaderMapIndex], *Results);
							}
							ShaderMapJobs.Remove(ShaderMapIdsToFinishCompiling[ShaderMapIndex]);
						}
						else
						{
							Results->CheckIfHung();
							NumPendingJobs += Results->NumPendingJobs.GetValue();
						}
					}
				}
			}

			if (NumPendingJobs > 0)
			{
				const float SleepTime =.01f;
				
				// We need to manually tick the Distributed build controller while the game thread is blocked
				// otherwise we can get stuck in a infinite loop waiting for jobs that never will be done
				// because for example, some controllers depend on the HTTP module which needs to be ticked in the main thread
				if (BuildDistributionController && IsInGameThread())
				{
					BuildDistributionController->Tick(SleepTime);
				}

				// Progress the slow task with how many jobs we've completed since last tick.  Update the slow task message with the current number of pending jobs
				// we are waiting on.
				const int32 CompletedJobsSinceLastTick = NumPreviousPendingJobs - NumPendingJobs;
				SlowTask.EnterProgressFrame(CompletedJobsSinceLastTick, FText::Format(LOCTEXT("BlockOnShaderMapCompletion", "Compiling Shaders ({0})"), NumPendingJobs));
				NumPreviousPendingJobs = NumPendingJobs;
				
				// Yield CPU time while waiting
				FPlatformProcess::Sleep(SleepTime);

				// Flush threaded logs around every 500ms or so based on Sleep of 0.01f seconds above
				if (++LogCounter > 50)
				{
					LogCounter = 0;
					GLog->FlushThreadedLogs(EOutputDeviceRedirectorFlushOptions::Async);
				}
			}
		} 
		while (NumPendingJobs > 0);
	}
	else
	{
		int32 NumActiveWorkers = 0;
		do 
		{
			for (const TUniquePtr<FShaderCompileThreadRunnableBase>& Thread : Threads)
			{
				NumActiveWorkers = Thread->CompilingLoop();
			}
		} 
		while (NumActiveWorkers > 0);

		check(AllJobs.GetNumPendingJobs() == 0);

		for (int32 ShaderMapIndex = 0; ShaderMapIndex < ShaderMapIdsToFinishCompiling.Num(); ShaderMapIndex++)
		{
			const FPendingShaderMapCompileResultsPtr* ResultsPtr = ShaderMapJobs.Find(ShaderMapIdsToFinishCompiling[ShaderMapIndex]);

			if (ResultsPtr)
			{
				const FShaderMapCompileResults* Results = *ResultsPtr;
				check(Results->NumPendingJobs.GetValue() == 0);
				check(Results->FinishedJobs.Num() > 0);

				AddCompiledResults(CompiledShaderMaps, ShaderMapIdsToFinishCompiling[ShaderMapIndex], *Results);
				ShaderMapJobs.Remove(ShaderMapIdsToFinishCompiling[ShaderMapIndex]);
			}
		}
	}

	UpdateNumRemainingAssets();
}

void FShaderCompilingManager::BlockOnAllShaderMapCompletion(TMap<int32, FShaderMapFinalizeResults>& CompiledShaderMaps)
{
	// never block if no compiling, just in case
	if (!AllowShaderCompiling())
	{
		return;
	}

	TRACE_CPUPROFILER_EVENT_SCOPE(FShaderCompilingManager::BlockOnAllShaderMapCompletion);

	COOK_STAT(FScopedDurationAtomicTimer BlockingTimer(ShaderCompilerCookStats::BlockingTimeSec));
	if (bAllowAsynchronousShaderCompiling)
	{
		// Calculate how many shader jobs there are total to provide the slow task with the correct amount of work.
		int NumJobs = 0;
		{
			FScopeLock Lock(&CompileQueueSection);
			for (TMap<int32, FPendingShaderMapCompileResultsPtr>::TIterator It(ShaderMapJobs); It; ++It)
			{
				FShaderMapCompileResults* Results = It.Value();
				NumJobs += Results->NumPendingJobs.GetValue();
			}
		}

		FScopedSlowTask SlowTask(NumJobs, FText::Format(LOCTEXT("BlockOnAllShaderMapCompletion", "Compiling Shaders ({0})"), NumJobs), NumJobs && GIsEditor && !IsRunningCommandlet());
		if (NumJobs > 0)
		{
			SlowTask.MakeDialog(false, true);
		}

		int32 NumPendingJobs = 0;
		// Keep track of previous number of pending jobs so we can update the slow task with the amount of work done.
		int32 NumPreviousPendingJobs = NumJobs;

		do 
		{
			NumPendingJobs = 0;
			{
				// Lock CompileQueueSection so we can access the input and output queues
				FScopeLock Lock(&CompileQueueSection);

				int32 ShaderMapIdx = 0;
				for (TMap<int32, FPendingShaderMapCompileResultsPtr>::TIterator It(ShaderMapJobs); It; ++It)
				{
					FShaderMapCompileResults* Results = It.Value();

					if (Results->NumPendingJobs.GetValue() == 0)
					{
						AddCompiledResults(CompiledShaderMaps, It.Key(), *Results);
						It.RemoveCurrent();
					}
					else
					{
						Results->CheckIfHung();
						NumPendingJobs += Results->NumPendingJobs.GetValue();
					}
				}
			}

			if (NumPendingJobs > 0)
			{
				const float SleepTime =.01f;
				
				// We need to manually tick the Distributed build controller while the game thread is blocked
				// otherwise we can get stuck in a infinite loop waiting for jobs that never will be done
				// because for example, some controllers depend on the HTTP module which needs to be ticked in the main thread
				if (BuildDistributionController && IsInGameThread())
				{
					BuildDistributionController->Tick(SleepTime);
				}

				// Progress the slow task with how many jobs we've completed since last tick.  Update the slow task message with the current number of pending jobs
				// we are waiting on.
				const int32 CompletedJobsSinceLastTick = NumPreviousPendingJobs - NumPendingJobs;
				SlowTask.EnterProgressFrame(CompletedJobsSinceLastTick, FText::Format(LOCTEXT("BlockOnAllShaderMapCompletion", "Compiling Shaders ({0})"), NumPendingJobs));
				NumPreviousPendingJobs = NumPendingJobs;
				
				// Yield CPU time while waiting
				FPlatformProcess::Sleep(SleepTime);
			}
		} 
		while (NumPendingJobs > 0);
	}
	else
	{
		int32 NumActiveWorkers = 0;
		do 
		{
			for (const TUniquePtr<FShaderCompileThreadRunnableBase>& Thread : Threads)
			{
				NumActiveWorkers = Thread->CompilingLoop();
			}

			for (TMap<int32, FPendingShaderMapCompileResultsPtr>::TIterator It(ShaderMapJobs); It; ++It)
			{
				FShaderMapCompileResults* Results = It.Value();
				Results->CheckIfHung();
			}
		} 
		while (NumActiveWorkers > 0);

		check(AllJobs.GetNumPendingJobs() == 0);

		for (TMap<int32, FPendingShaderMapCompileResultsPtr>::TIterator It(ShaderMapJobs); It; ++It)
		{
			const FShaderMapCompileResults* Results = It.Value();
			check(Results->NumPendingJobs.GetValue()== 0);

			AddCompiledResults(CompiledShaderMaps, It.Key(), *Results);
			It.RemoveCurrent();
		}
	}

	UpdateNumRemainingAssets();
}

namespace
{
	void PropagateGlobalShadersToAllPrimitives()
	{
		// Re-register everything to work around FShader lifetime issues - it currently lives and dies with the
		// shadermap it is stored in, while cached MDCs can reference its memory. Re-registering will
		// re-create the cache.
		TRACE_CPUPROFILER_EVENT_SCOPE(PropagateGlobalShadersToAllPrimitives);

		FObjectCacheContextScope ObjectCacheScope;
		TSet<FSceneInterface*> ScenesToUpdate;
		TIndirectArray<FComponentRecreateRenderStateContext> ComponentContexts;
		for (IPrimitiveComponent* PrimitiveComponentInterface : ObjectCacheScope.GetContext().GetPrimitiveComponents())
		{
			if (PrimitiveComponentInterface->IsRenderStateCreated())
			{
				ComponentContexts.Add(new FComponentRecreateRenderStateContext(PrimitiveComponentInterface, &ScenesToUpdate));
#if WITH_EDITOR
				if (UPrimitiveComponent* PrimitiveComponent = PrimitiveComponentInterface->GetUObject<UPrimitiveComponent>())
				{
					if (PrimitiveComponent->HasValidSettingsForStaticLighting(false))
					{
						FStaticLightingSystemInterface::OnPrimitiveComponentUnregistered.Broadcast(PrimitiveComponent);
						FStaticLightingSystemInterface::OnPrimitiveComponentRegistered.Broadcast(PrimitiveComponent);
					}
				}
#endif
			}
		}

		UpdateAllPrimitiveSceneInfosForScenes(ScenesToUpdate);
		ComponentContexts.Empty();
		UpdateAllPrimitiveSceneInfosForScenes(ScenesToUpdate);
	}
}

void FShaderCompilingManager::ProcessCompiledShaderMaps(
	TMap<int32, FShaderMapFinalizeResults>& CompiledShaderMaps, 
	float TimeBudget)
{
	// never process anything if not allowed, just in case
	if (!AllowShaderCompiling())
	{
		return;
	}

#if WITH_EDITOR
	TRACE_CPUPROFILER_EVENT_SCOPE(FShaderCompilingManager::ProcessCompiledShaderMaps);

	TMap<TRefCountPtr<FMaterial>, TRefCountPtr<FMaterialShaderMap>> MaterialsToUpdate;
	TArray<TRefCountPtr<FMaterial>> MaterialsToReleaseCompilingId;

	// Process compiled shader maps in FIFO order, in case a shader map has been enqueued multiple times,
	// Which can happen if a material is edited while a background compile is going on
	for (TMap<int32, FShaderMapFinalizeResults>::TIterator ShaderMapResultIter(CompiledShaderMaps); ShaderMapResultIter; ++ShaderMapResultIter)
	{
		const uint32 CompilingId = ShaderMapResultIter.Key();

		FShaderMapFinalizeResults& CompileResults = ShaderMapResultIter.Value();
		TArray<FShaderCommonCompileJobPtr>& FinishedJobs = CompileResults.FinishedJobs;

		if (CompileResults.bSkipResultProcessing)
		{
			ShaderMapResultIter.RemoveCurrent();
			continue;
		}

		TRefCountPtr<FMaterialShaderMap> CompilingShaderMap = FMaterialShaderMap::FindCompilingShaderMap(CompilingId);

		if (CompilingShaderMap)
		{
			TArray<TRefCountPtr<FMaterial>>& MaterialDependencies = CompilingShaderMap->CompilingMaterialDependencies;
			FShaderDiagnosticInfo ErrorInfo(FinishedJobs);

			bool bSuccess = true;
			for (int32 JobIndex = 0; JobIndex < FinishedJobs.Num(); JobIndex++)
			{
				FShaderCommonCompileJob& CurrentJob = *FinishedJobs[JobIndex];

				if (FShaderCompileJob* SingleJob = CurrentJob.GetSingleShaderJob())
				{
					const bool bCheckSucceeded = CheckSingleJob(*SingleJob, ErrorInfo.UniqueErrors);
					bSuccess = bCheckSucceeded && bSuccess;
				}
				else if (FShaderPipelineCompileJob* PipelineJob = CurrentJob.GetShaderPipelineJob())
				{
					for (int32 Index = 0; Index < PipelineJob->StageJobs.Num(); ++Index)
					{
						const bool bCheckSucceeded = CheckSingleJob(*PipelineJob->StageJobs[Index], ErrorInfo.UniqueErrors);
						bSuccess = PipelineJob->StageJobs[Index]->bSucceeded && bCheckSucceeded && bSuccess;
					}
				}
				else
				{
					checkf(0, TEXT("FShaderCommonCompileJob::Type=%d is not a valid type for a shader compile job"), (int32)CurrentJob.Type);
				}
			}

			if (bSuccess)
			{
				int32 JobIndex = 0;
				if (FinishedJobs.Num() > 0)
				{
					CompilingShaderMap->ProcessCompilationResults(FinishedJobs, JobIndex, TimeBudget);
					{
						FScopeLock Lock(&CompileQueueSection);
						for (int32 i = 0; i < JobIndex; ++i)
						{
							ReleaseJob(FinishedJobs[i]);
						}
					}
					FinishedJobs.RemoveAt(0, JobIndex);
				}
			}

			if (!bSuccess || FinishedJobs.Num() == 0)
			{
				ShaderMapResultIter.RemoveCurrent();
			}

			FMaterialShaderMap* ShaderMapToUseForRendering = nullptr;

			int32 NumIncompleteMaterials = 0;
			int32 MaterialIndex = 0;
			
			FMaterial* SingleMaterial = MaterialDependencies.Num() > 0 ? MaterialDependencies[0] : nullptr;
			bool bRequiredComplete = false;

			while (MaterialIndex < MaterialDependencies.Num())
			{
				FMaterial* Material = MaterialDependencies[MaterialIndex];
				check(Material->GetGameThreadCompilingShaderMapId() == CompilingShaderMap->GetCompilingId());
				bRequiredComplete |= Material->IsRequiredComplete();

				//Material->RemoveOutstandingCompileId(ShaderMap->CompilingId);

				bool bReleaseCompilingId = false;

				// Only process results that still match the ID which requested a compile
				// This avoids applying shadermaps which are out of date and a newer one is in the async compiling pipeline
				if (Material->GetMaterialId() != CompilingShaderMap->GetShaderMapId().BaseMaterialId)
				{
					bReleaseCompilingId = true;
				}
				else if (bSuccess)
				{
					bool bIsComplete = CompilingShaderMap->IsComplete(Material, true) && (CompilingShaderMap->CompilingMaterialNumExternalDependencies == 0);

					// If running a cook, only process complete shader maps, as there's no rendering of partially complete shader maps to worry about.
					if (bIsComplete || IsRunningCookCommandlet() == false || bAllowForIncompleteShaderMaps)
					{
						if (ShaderMapToUseForRendering == nullptr)
						{
							// Make a clone of the compiling shader map to use for rendering
							// This will allow rendering to proceed with the clone, while async compilation continues to potentially update the compiling shader map
							double StartTime = FPlatformTime::Seconds();
							ShaderMapToUseForRendering = CompilingShaderMap->AcquireFinalizedClone();
							TimeBudget -= (FPlatformTime::Seconds() - StartTime);
						}

						MaterialsToUpdate.Add(Material, ShaderMapToUseForRendering);
					}

					if (bIsComplete)
					{
						bReleaseCompilingId = true;
					}
					else
					{
						++NumIncompleteMaterials;
					}

					if (ErrorInfo.UniqueWarnings.Num() > 0)
					{
						const FName ShaderPlatformName = LegacyShaderPlatformToShaderFormat(CompilingShaderMap->GetShaderPlatform());
						if (ShouldShowShaderWarnings())
						{
							UE_LOGF(LogShaderCompilers, Warning, "Warnings while compiling Material %ls for platform %ls:",
								*Material->GetDebugName(), *ShaderPlatformName.ToString());
						}
						else
						{
							UE_LOGF(LogShaderCompilers, Display, "Diagnostic messages while compiling Material %ls for platform %ls:",
								*Material->GetDebugName(), *ShaderPlatformName.ToString());
						}

						for (const FString& UniqueWarning : ErrorInfo.UniqueWarnings)
						{
							// Use structured logs to avoid build machine string matching heuristic that will cause a cook to fail
							if (ShouldShowShaderWarnings())
							{
								UE_LOGFMT(LogShaderCompilers, Warning, "  {DiagnosticMessage}", UniqueWarning);
							}
							else
							{
								UE_LOGFMT(LogShaderCompilers, Display, "  {DiagnosticMessage}", UniqueWarning);
							}
						}
					}
				}
				else
				{
					bReleaseCompilingId = true;
					// Propagate error messages
					Material->CompileErrors = ErrorInfo.UniqueErrors;

					MaterialsToUpdate.Add(Material, nullptr);

					if (Material->IsDefaultMaterial())
					{
						FString ErrorString = FString::Printf(TEXT("Failed to compile default material %s!\n"), *Material->GetBaseMaterialPathName());

						// Log the errors unsuppressed before the fatal error, so it's always obvious from the log what the compile error was
						for (const FString& UniqueError : ErrorInfo.UniqueErrors)
						{
							ErrorString += FString::Printf(TEXT("%s\n\n"), *UniqueError);
						}

						if (AreShaderErrorsFatal())
						{
							// Assert if a default material could not be compiled, since there will be nothing for other failed materials to fall back on.
							UE_LOGF(LogShaderCompilers, Fatal, "\n%ls", *ErrorString);
						}
						else
						{
							UE_LOGF(LogShaderCompilers, Error, "\n%ls", *ErrorString);
						}
					}
					
					FString ErrorString = FString::Printf(TEXT("Failed to compile Material %s for platform %s, Default Material will be used in game.\n"),
						*Material->GetDebugName(), *LegacyShaderPlatformToShaderFormat(CompilingShaderMap->GetShaderPlatform()).ToString());

					for (const FString& UniqueError : ErrorInfo.UniqueErrors)
					{
						ErrorString += FString::Printf(TEXT("%s\n"), *UniqueError);
					}

					// Use structured logs to avoid build machine string matching heuristic that will cause a cook to fail
					UE_LOGFMT(LogShaderCompilers, Warning, "{DiagnosticMessage}", ErrorString);
				}

				if (bReleaseCompilingId)
				{
					check(Material->GameThreadCompilingShaderMapId != 0u);
					Material->GameThreadCompilingShaderMapId = 0u;
					Material->GameThreadPendingCompilerEnvironment.SafeRelease();
					MaterialDependencies.RemoveAt(MaterialIndex);
					if (MaterialDependencies.Num() == 0 && CompilingShaderMap->CompilingMaterialNumExternalDependencies == 0)
					{
						CompilingShaderMap->ReleaseCompilingId();
					}
					MaterialsToReleaseCompilingId.Add(Material);
				}
				else
				{
					++MaterialIndex;
				}
			}

			if (NumIncompleteMaterials == 0 && (IsMaterialMapDDCEnabled() || bRequiredComplete))
			{
				CompilingShaderMap->bCompiledSuccessfully = bSuccess;
				CompilingShaderMap->bCompilationFinalized = true;
				if (ShaderMapToUseForRendering)
				{
					// ShaderMapToUseForRendering is only initialized inside the loop over material dependencies,
					// so it's safe to assume that SingleMaterial has been set (a material is needed to construct 
					// the FMaterialShaderParameters struct which is in turn needed to build the DDC key).
					check(SingleMaterial != nullptr);
					ShaderMapToUseForRendering->bCompiledSuccessfully = true;
					ShaderMapToUseForRendering->bCompilationFinalized = true;
					if (ShaderMapToUseForRendering->bIsPersistent)
					{
						ShaderMapToUseForRendering->SaveToDerivedDataCache(FConservativeShaderParameters(SingleMaterial));
					}
				}

				CompilingShaderMap->ReleaseCompilingId();
			}

			if (TimeBudget < 0)
			{
				break;
			}
		}
		else
		{
			if (CompilingId == GlobalShaderMapId)
			{
				ProcessCompiledGlobalShaders(FinishedJobs);
				PropagateGlobalShadersToAllPrimitives();
			}

			// ShaderMap was removed from compiling list or is being used by another type of shader map which is maintaining a reference
			// to the results, either way the job can be released
			{
				FScopeLock Lock(&CompileQueueSection);
				for (FShaderCommonCompileJobPtr& Job : FinishedJobs)
				{
					ReleaseJob(Job);
				}
			}
			ShaderMapResultIter.RemoveCurrent();
		}
	}

	if (MaterialsToReleaseCompilingId.Num() > 0)
	{
		ENQUEUE_RENDER_COMMAND(ReleaseCompilingShaderMapIds)([MaterialsToReleaseCompilingId = MoveTemp(MaterialsToReleaseCompilingId)](FRHICommandListImmediate& RHICmdList)
		{
			for (FMaterial* Material : MaterialsToReleaseCompilingId)
			{
				check(Material->RenderingThreadCompilingShaderMapId != 0u);
				Material->RenderingThreadCompilingShaderMapId = 0u;
				Material->RenderingThreadPendingCompilerEnvironment.SafeRelease();
			}
		});
	}

	if (MaterialsToUpdate.Num() > 0)
	{
		FMaterial::SetShaderMapsOnMaterialResources(MaterialsToUpdate);

		for (const auto& It : MaterialsToUpdate)
		{
			It.Key->NotifyCompilationFinished();
		}

		if (FApp::CanEverRender())
		{
			// This empties MaterialsToUpdate, see the comment inside the function for the reason.
			PropagateMaterialChangesToPrimitives(MaterialsToUpdate);

			FEditorSupportDelegates::RedrawAllViewports.Broadcast();
		}
	}

	UpdateNumRemainingAssets();
#endif // WITH_EDITOR
}

void FShaderCompilingManager::PropagateMaterialChangesToPrimitives(TMap<TRefCountPtr<FMaterial>, TRefCountPtr<FMaterialShaderMap>>& MaterialsToUpdate)
{
	// don't perform any work if no compiling
	if (!AllowShaderCompiling())
	{
		return;
	}

	TSet<FSceneInterface*> ScenesToUpdate;
	FObjectCacheContextScope ObjectCacheScope;
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FShaderCompilingManager::PropagateMaterialChangesToPrimitives);

		TArray<UMaterialInterface*> UpdatedMaterials;
		for (TMap<TRefCountPtr<FMaterial>, TRefCountPtr<FMaterialShaderMap>>::TConstIterator MaterialIt(MaterialsToUpdate); MaterialIt; ++MaterialIt)
		{
			FMaterial* UpdatedMaterial = MaterialIt.Key();
			UpdatedMaterials.Add(UpdatedMaterial->GetMaterialInterface());
		}

		for (IPrimitiveComponent* PrimitiveComponent : ObjectCacheScope.GetContext().GetPrimitivesAffectedByMaterials(UpdatedMaterials))
		{
			PrimitiveComponent->MarkRenderStateDirty();
		}
	}

	// Recreating the render state for the primitives may end up recreating the material resources if some materials are missing some usage flags.
	// For example, if some materials are not marked as used with static lighting and we build lightmaps, UMaterialInstance::CheckMaterialUsage
	// will catch the problem and try to set the flag. However, since MaterialsToUpdate stores smart pointers, the material resources will have
	// a refcount of 2, so the FMaterial destructor will trigger a check failure because the refcount doesn't reach 0. Empty this map before
	// recreating the render state to allow resources to be deleted cleanly.
	MaterialsToUpdate.Empty();
}


/**
 * Shutdown the shader compile manager
 * this function should be used when ending the game to shutdown shader compile threads
 * will not complete current pending shader compilation
 */
void FShaderCompilingManager::Shutdown()
{
	// Shutdown has been moved to the destructor because the shader compiler lifetime is expected to
	// be longer than other asset compilers, otherwise niagara compilations might get stuck.
}

void FShaderCompilingManager::PrintStats()
{
	FShaderCompilerStats LocalStats;
	GetLocalStats(LocalStats);
	LocalStats.WriteStatSummary();
}

void FShaderCompilingManager::GetLocalStats(FShaderCompilerStats& OutStats) const
{
	if (GShaderCompilerStats)
	{
		OutStats.Aggregate(*GShaderCompilerStats);
		AllJobs.GetCachingStats(OutStats);
	}
}

FShaderCompileMemoryUsage FShaderCompilingManager::GetExternalMemoryUsage()
{
	FShaderCompileMemoryUsage TotalMemoryUsage{};
	for (const TUniquePtr<FShaderCompileThreadRunnableBase>& ThreadPtr : Threads)
	{
		FShaderCompileMemoryUsage MemoryUsage = ThreadPtr->GetExternalWorkerMemoryUsage();
		TotalMemoryUsage.VirtualMemory += MemoryUsage.VirtualMemory;
		TotalMemoryUsage.PhysicalMemory += MemoryUsage.PhysicalMemory;
	}
	return TotalMemoryUsage;
}

static void BuildErrorStringAndReport(const FShaderDiagnosticInfo& DiagInfo, FString& ErrorString)
{
	bool bReportedDebugInfo = false;

	for (int32 ErrorIndex = 0; ErrorIndex < DiagInfo.UniqueErrors.Num(); ErrorIndex++)
	{
		FString UniqueErrorString = DiagInfo.UniqueErrors[ErrorIndex] + TEXT("\n\n");

		if (FPlatformMisc::IsDebuggerPresent())
		{
			// Using OutputDebugString to avoid any text getting added before the filename,
			// Which will throw off VS.NET's ability to take you directly to the file and line of the error when double clicking it in the output window.
			FPlatformMisc::LowLevelOutputDebugString(*UniqueErrorString);
		}
		else
		{
			UE_LOGF(LogShaderCompilers, Warning, "%ls", *UniqueErrorString);
		}

		ErrorString += UniqueErrorString;
	}
}

bool FShaderCompilingManager::HandlePotentialRetry(TMap<int32, FShaderMapFinalizeResults>& CompletedShaderMaps)
{
#if WITH_EDITORONLY_DATA
	TArray<FShaderCommonCompileJobPtr> ReissueForDebugJobs;
	TArray<FShaderCommonCompileJobPtr> ReissueJobs;
	TArray<int32> ReissueMapIds;
	bool bInteractiveRetry = false;

	for (TMap<int32, FShaderMapFinalizeResults>::TIterator It(CompletedShaderMaps); It; ++It)
	{
		bool bAnyToReissue = false;

		FShaderMapFinalizeResults& Results = It.Value();
		// Handle retries based on r.DumpShaderDebugInfo=2/3 first; if any jobs require this we skip the interactive retry prompt until after such jobs run
		// (so the retry prompt will contain the debug info path)
		for (int32 FinishedJobIndex = 0; FinishedJobIndex < Results.FinishedJobs.Num(); ++FinishedJobIndex)
		{
			FShaderCommonCompileJobPtr Job = Results.FinishedJobs[FinishedJobIndex];
			bool bEnableDebug = false;
			Job->ForEachSingleShaderJob([&bEnableDebug](FShaderCompileJob& SingleJob)
				{
					// reissue the job if either we want to dump debug info for it, or a retry was requested above for any failed jobs
					bEnableDebug = GShaderCompilingManager->ShouldRecompileToDumpShaderDebugInfo(SingleJob);
				});

			if (bEnableDebug)
			{
				bAnyToReissue = true;
				ReissueForDebugJobs.Add(Job);
				Results.FinishedJobs.RemoveAt(FinishedJobIndex--, EAllowShrinking::No);
			}
		}


		// interactive retries - prompt for global/default shaders that have errors, if we're not reissuing any jobs to dump debug info first
		if (!FApp::IsUnattended() && !Results.bAllJobsSucceeded && ReissueForDebugJobs.IsEmpty())
		{
			bool bSpecialEngineMaterial = false;

			const FMaterialShaderMap* ShaderMap = FMaterialShaderMap::FindCompilingShaderMap(It.Key());
			if (ShaderMap)
			{
				for (const FMaterial* Material : ShaderMap->CompilingMaterialDependencies)
				{
					if (Material->IsSpecialEngineMaterial())
					{
						bSpecialEngineMaterial = true;
						break;
					}
				}
			}

			if (UE_LOG_ACTIVE(LogShaders, Log)
				// Always log detailed errors when a special engine material or global shader fails to compile, as those will be fatal errors
				|| bSpecialEngineMaterial
				|| It.Key() == GlobalShaderMapId)
			{
				TArray<FShaderCommonCompileJobPtr>& CompleteJobs = Results.FinishedJobs;
				FShaderDiagnosticInfo ShaderDiagInfo(CompleteJobs);

				const TCHAR* MaterialName = ShaderMap ? ShaderMap->GetFriendlyName() : TEXT("global shaders");
				FString ErrorString = FString::Printf(TEXT("%i Shader compiler errors compiling %s for platform %s:"), ShaderDiagInfo.UniqueErrors.Num(), MaterialName, *ShaderDiagInfo.TargetShaderPlatformString);
				UE_LOGF(LogShaderCompilers, Warning, "%ls", *ErrorString);
				ErrorString += TEXT("\n\n");

				bool bAnyErrorLikelyToBeCodeError = false;
				for (const FShaderCommonCompileJob* Job : ShaderDiagInfo.ErrorJobs)
				{
					bAnyErrorLikelyToBeCodeError |= Job->bErrorsAreLikelyToBeCode;
				}

				BuildErrorStringAndReport(ShaderDiagInfo, ErrorString);

				if ((bAnyErrorLikelyToBeCodeError || bPromptToRetryFailedShaderCompiles || bSpecialEngineMaterial) 
					&& !bInteractiveRetry) // break/prompt only for the first shadermap which encountered an error, to avoid dialog spam
				{
					// Use debug break in debug with the debugger attached, otherwise message box
					if (bDebugBreakOnPromptToRetryShaderCompile && FPlatformMisc::IsDebuggerPresent())
					{
						// A shader compile error has occurred, see the debug output for information.
						// Double click the errors in the VS.NET output window and the IDE will take you directly to the file and line of the error.
						// Check ErrorJobs for more state on the failed shaders, for example in-memory includes like Material.usf
						UE_DEBUG_BREAK();
						// Set GRetryShaderCompilation to true in the debugger to enable retries in debug
						// NOTE: MaterialTemplate.usf will not be reloaded when retrying!
						bInteractiveRetry = GRetryShaderCompilation;
					}
					else
					{
						if (FPlatformMisc::MessageBoxExt(EAppMsgType::YesNo, *FText::Format(NSLOCTEXT("UnrealEd", "Error_RetryShaderCompilation", "{0}\nRetry compilation?"),
							FText::FromString(ErrorString)).ToString(), TEXT("Error")) == EAppReturnType::Type::Yes)
						{
							bInteractiveRetry = true;
						}
					}
				}

				if (bInteractiveRetry)
				{
					bAnyToReissue = true;
					for (int32 FinishedJobIndex = 0; FinishedJobIndex < Results.FinishedJobs.Num(); ++FinishedJobIndex)
					{
						FShaderCommonCompileJobPtr Job = Results.FinishedJobs[FinishedJobIndex];
						if (!Job->bSucceeded)
						{
							ReissueJobs.Add(Job);
							Results.FinishedJobs.RemoveAt(FinishedJobIndex--, EAllowShrinking::No);
						}
					}
				}
			}
		}
		if (bAnyToReissue)
		{
			ReissueMapIds.Add(It.Key());
			// reset flag indicating failed jobs for a map when reissuing; we will always re-issue all failed jobs if we reissue any,
			// so the ones remaining in the results array should all be successful
			It.Value().bAllJobsSucceeded = true;
		}
	}

	if (ReissueForDebugJobs.Num())
	{
		check(ReissueJobs.IsEmpty());
		for (const FShaderCommonCompileJobPtr& DebugJob : ReissueForDebugJobs)
		{
			// if any stage of a pipeline job is reissued to dump debug info, we need to enable debug info on all stages of the job
			// since the debug information relies on debug source files existing for every stage
			FShaderCompileInternalUtilities::EnableDumpDebugInfoForRetry(*DebugJob);
		}

		ReissueJobs = MoveTemp(ReissueForDebugJobs);
	}

	if (ReissueJobs.Num())
	{
		// Flush the shader file cache so that any changes will be propagated.
		FlushShaderFileCache();

		// Reset outputs
		for (const FShaderCommonCompileJobPtr& CurrentJob : ReissueJobs)
		{
			// NOTE: Changes to MaterialTemplate.usf before retrying won't work, because the entry for Material.usf in CurrentJob.Environment.IncludeFileNameToContentsMap isn't reset
			CurrentJob->ForEachSingleShaderJob(
				[](FShaderCompileJob& SingleJob)
				{
					SingleJob.Output = FShaderCompilerOutput();
					SingleJob.PreprocessOutput = FShaderPreprocessOutput();
					SingleJob.JobStatusPtr->Reset();
				}
			);
			
			// Reset DDC query request owner
			CurrentJob->RequestOwner.Reset();
			CurrentJob->JobStatusPtr->Reset();
			CurrentJob->bInputHashSet = false;
			// Need to force reissued jobs to skip the cache queries otherwise jobs with warnings will just be cache hits and not actually recompile
			// (since debug info being enabled for a job intentionally does not affect the cached key/cached results)
			CurrentJob->bBypassCache = true;
		}

		// Submit all the jobs which we want to recompile
		SubmitJobs(ReissueJobs, FString(), FString());

		// Block until the shader maps with reissued jobs have been compiled again (this may include new jobs since submission could
		// be occurring in parallel on other threads)
		BlockOnShaderMapCompletion(ReissueMapIds, CompletedShaderMaps);

		return true;
	}
#endif	//WITH_EDITORONLY_DATA
	return false;
}

void FShaderMapCompileResults::CheckIfHung()
{
	if (!bIsHung)
	{
		double DurationSoFar = FPlatformTime::Seconds() - TimeStarted;
		if (DurationSoFar >= static_cast<double>(GShaderMapCompilationTimeout))
		{
			bIsHung = true;
			// always produce an error message first, even if going to crash, as the automation controller does not seem to be picking up Fatal messages
			UE_LOGF(LogShaderCompilers, Error, "Hung shadermap detected, time spent compiling: %f seconds, NumPendingJobs: %d, FinishedJobs: %d",
				DurationSoFar,
				NumPendingJobs.GetValue(),
				FinishedJobs.Num()
			);

			if (GCrashOnHungShaderMaps)
			{
				// Dump all pending compile jobs that are assigned to this shader map before we crash with the following Fatal error
				GShaderCompilingManager->DumpDebugInfoForEachPendingJob([this](FShaderCommonCompileJob* Job) -> bool {
					if (Job->PendingShaderMap.GetReference() == this && !Job->bSucceeded)
					{
						Job->ForEachSingleShaderJob([](FShaderCompileJob& SingleJob) -> void {
							const double JobDuration = FPlatformTime::Seconds() - SingleJob.TimeAssignedToExecution;
							UE_LOGF(LogShaderCompilers, Log, "Job [%ls] pending for %.1f seconds", *SingleJob.Input.GenerateShaderName(), JobDuration);
						});
						return true;
					}
					return false;
				});

				UE_LOGF(LogShaderCompilers, Fatal, "Crashing on a hung shadermap, time spent compiling: %f seconds, NumPendingJobs: %d, FinishedJobs: %d",
					DurationSoFar,
					NumPendingJobs.GetValue(),
					FinishedJobs.Num()
				);
			}
		}
	}
}

void FShaderCompilingManager::CancelCompilation(const TCHAR* MaterialName, const TArray<int32>& ShaderMapIdsToCancel)
{
	// nothing to cancel here, just in case
	if (!AllowShaderCompiling())
	{
		return;
	}

	check(IsInGameThread());
	check(!FPlatformProperties::RequiresCookedData());

	// Lock CompileQueueSection so we can access the input and output queues
	FScopeLock Lock(&CompileQueueSection);

	int32 TotalNumJobsRemoved = 0;
	for (int32 IdIndex = 0; IdIndex < ShaderMapIdsToCancel.Num(); ++IdIndex)
	{
		int32 MapIdx = ShaderMapIdsToCancel[IdIndex];
		if (const FPendingShaderMapCompileResultsPtr* ResultsPtr = ShaderMapJobs.Find(MapIdx))
		{
			const int32 NumJobsRemoved = AllJobs.RemoveAllPendingJobsWithId(MapIdx);
	
			TotalNumJobsRemoved += NumJobsRemoved;

			FShaderMapCompileResults* ShaderMapJob = *ResultsPtr;
			const int32 PrevNumPendingJobs = ShaderMapJob->NumPendingJobs.Subtract(NumJobsRemoved);
			check(PrevNumPendingJobs >= NumJobsRemoved);

			// The shader map job result should be skipped since it is out of date.
			ShaderMapJob->bSkipResultProcessing = true;
		
			if (PrevNumPendingJobs == NumJobsRemoved && ShaderMapJob->FinishedJobs.Num() == 0)
			{
				//We've removed all the jobs for this shader map so remove it.
				ShaderMapJobs.Remove(MapIdx);
			}
		}

		// Don't continue finalizing once compilation has been canceled
		// the CompilingId has been removed from ShaderMapsBeingCompiled, which will cause crash when attempting to do any further processing
		const int32 NumPendingRemoved = PendingFinalizeShaderMaps.Remove(MapIdx);
	}

	if (TotalNumJobsRemoved > 0)
	{
		UE_LOGF(LogShaders, Display, "CancelCompilation %ls, Removed %d jobs", MaterialName ? MaterialName : TEXT(""), TotalNumJobsRemoved);
	}
}

void FShaderCompilingManager::FinishCompilation(const TCHAR* MaterialName, const TArray<int32>& ShaderMapIdsToFinishCompiling)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FShaderCompilingManager::FinishCompilation);

	// nothing to do
	if (!AllowShaderCompiling())
	{
		return;
	}

	check(IsInGameThread());
	check(!FPlatformProperties::RequiresCookedData());
	const double StartTime = FPlatformTime::Seconds();

	FText StatusUpdate;
	if ( MaterialName != nullptr)
	{
		FFormatNamedArguments Args;
		Args.Add( TEXT("MaterialName"), FText::FromString( MaterialName ) );
		StatusUpdate = FText::Format( NSLOCTEXT("ShaderCompilingManager", "CompilingShadersForMaterialStatus", "Compiling shaders: {MaterialName}..."), Args );
	}
	else
	{
		StatusUpdate = NSLOCTEXT("ShaderCompilingManager", "CompilingShadersStatus", "Compiling shaders...");
	}

	FScopedSlowTask SlowTask(1, StatusUpdate, GIsEditor && !IsRunningCommandlet());
	SlowTask.EnterProgressFrame(1);

	TMap<int32, FShaderMapFinalizeResults> CompiledShaderMaps;
	CompiledShaderMaps.Append( PendingFinalizeShaderMaps );
	PendingFinalizeShaderMaps.Empty();
	BlockOnShaderMapCompletion(ShaderMapIdsToFinishCompiling, CompiledShaderMaps);

	bool bRetry = false;
	do 
	{
		bRetry = HandlePotentialRetry(CompiledShaderMaps);
	} 
	while (bRetry);

	ProcessCompiledShaderMaps(CompiledShaderMaps, FLT_MAX);
	check(CompiledShaderMaps.Num() == 0);

	const double EndTime = FPlatformTime::Seconds();

	UE_LOGF(LogShaders, Verbose, "FinishCompilation %ls %.3fs", MaterialName ? MaterialName : TEXT(""), (float)(EndTime - StartTime));
}

void FShaderCompilingManager::FinishAllCompilation()
{
	// No-op if nothing changed since last call, but required at the beginning for LastNumRemainingAssets to
	// be updated so that the subsequent call can notice that where was something to compile and the amount is back to 0.
	UpdateNumRemainingAssets();

#if WITH_EDITOR
	// This is here for backward compatibility since textures are most probably expected to be ready too.
	FTextureCompilingManager::Get().FinishAllCompilation();
#endif

	TRACE_CPUPROFILER_EVENT_SCOPE(FShaderCompilingManager::FinishAllCompilation);
	check(IsInGameThread());
	check(!FPlatformProperties::RequiresCookedData());
	const double StartTime = FPlatformTime::Seconds();

	TMap<int32, FShaderMapFinalizeResults> CompiledShaderMaps;
	CompiledShaderMaps.Append( PendingFinalizeShaderMaps );
	PendingFinalizeShaderMaps.Empty();
	BlockOnAllShaderMapCompletion(CompiledShaderMaps);

	bool bRetry = false;
	do 
	{
		bRetry = HandlePotentialRetry(CompiledShaderMaps);
	} 
	while (bRetry);

	ProcessCompiledShaderMaps(CompiledShaderMaps, FLT_MAX);
	check(CompiledShaderMaps.Num() == 0);

	const double EndTime = FPlatformTime::Seconds();

	UpdateNumRemainingAssets();

	UE_LOGF(LogShaders, Verbose, "FinishAllCompilation %.3fs", (float)(EndTime - StartTime));
}

void FShaderCompilingManager::ProcessAsyncResults(bool bLimitExecutionTime, bool bBlockOnGlobalShaderCompletion)
{
	const float TimeSlice = bLimitExecutionTime ? ProcessGameThreadTargetTime : 0.f;
	ProcessAsyncResults(TimeSlice, bBlockOnGlobalShaderCompletion);
}

void FShaderCompilingManager::DumpDebugInfoForEachPendingJob(const FShaderCompileJobCallback& PendingJobCallback) const
{
	check(PendingJobCallback);

	for (const TUniquePtr<FShaderCompileThreadRunnableBase>& ShaderCompileThread : Threads)
	{
		ShaderCompileThread->ForEachPendingJob([this, &PendingJobCallback](FShaderCommonCompileJob* Job) -> bool {
			check(Job);
			if (PendingJobCallback(Job))
			{
				// Dump debug information for current job
				FShaderCompileInternalUtilities::DumpDebugInfo(*Job);
			}
			return true; // Always perform full iteration, don't exit early
		});
	}
}

void FShaderCompilingManager::ProcessAsyncResults(float TimeSlice, bool bBlockOnGlobalShaderCompletion)
{
	LLM_SCOPE_BYTAG(ShaderCompiler);

	TRACE_CPUPROFILER_EVENT_SCOPE(FShaderCompilingManager::ProcessAsyncResults)

	COOK_STAT(FScopedDurationAtomicTimer Timer(ShaderCompilerCookStats::ProcessAsyncResultsTimeSec));
	check(IsInGameThread());

	const double StartTime = FPlatformTime::Seconds();

	// No-op if nothing changed since last call, but required at the beginning for LastNumRemainingAssets to
	// be updated so that the subsequent call can notice that where was something to compile and the amount is back to 0.
	UpdateNumRemainingAssets();

	// Some controllers need to be manually ticked if the engine loop is not initialized or blocked
	// to do things like tick the HTTPModule.
	// Otherwise the results from the controller will never be processed.
	// We check for bBlockOnGlobalShaderCompletion because the BlockOnShaderMapCompletion methods already do this.
	if (!bBlockOnGlobalShaderCompletion && BuildDistributionController)
	{
		BuildDistributionController->Tick(0.0f);
	}

	// Block on global shaders before checking for shader maps to finalize
	// So if we block on global shaders for a long time, we will get a chance to finalize all the non-global shader maps completed during that time.
	if (bBlockOnGlobalShaderCompletion)
	{
		TArray<int32> ShaderMapId;
		ShaderMapId.Add(GlobalShaderMapId);

		// Block until the global shader map jobs are complete
		GShaderCompilingManager->BlockOnShaderMapCompletion(ShaderMapId, PendingFinalizeShaderMaps);
	}

	int32 NumCompilingShaderMaps = 0;

	{
		// Lock CompileQueueSection so we can access the input and output queues
		FScopeLock Lock(&CompileQueueSection);

		if (!bBlockOnGlobalShaderCompletion)
		{
			bCompilingDuringGame = true;
		}

		// Get all material shader maps to finalize
		//
		for (TMap<int32, FPendingShaderMapCompileResultsPtr>::TIterator It(ShaderMapJobs); It; ++It)
		{
			FPendingShaderMapCompileResultsPtr& Results = It.Value();
			if (Results->FinishedJobs.Num() > 0)
			{
				FShaderMapFinalizeResults& FinalizeResults = PendingFinalizeShaderMaps.FindOrAdd(It.Key());
				FinalizeResults.FinishedJobs.Append(Results->FinishedJobs);
				Results->FinishedJobs.Reset();
				FinalizeResults.bAllJobsSucceeded = FinalizeResults.bAllJobsSucceeded && Results->bAllJobsSucceeded;
			}

			checkf(Results->FinishedJobs.Num() == 0, TEXT("Failed to remove finished jobs, %d remain"), Results->FinishedJobs.Num());
			if (Results->NumPendingJobs.GetValue() == 0)
			{
				It.RemoveCurrent();
			}
		}

		NumCompilingShaderMaps = ShaderMapJobs.Num();
	}

	int32 NumPendingShaderMaps = PendingFinalizeShaderMaps.Num();

	if (PendingFinalizeShaderMaps.Num() > 0)
	{
		bool bRetry = false;
		do 
		{
			bRetry = HandlePotentialRetry(PendingFinalizeShaderMaps);
		} 
		while (bRetry);

		const float TimeBudget = TimeSlice > 0 ? TimeSlice : FLT_MAX;
		ProcessCompiledShaderMaps(PendingFinalizeShaderMaps, TimeBudget);
		check(TimeSlice > 0 || PendingFinalizeShaderMaps.Num() == 0);
	}


	if (bBlockOnGlobalShaderCompletion && TimeSlice <= 0 && !IsRunningCookCommandlet())
	{
		check(PendingFinalizeShaderMaps.Num() == 0);

		if (NumPendingShaderMaps - PendingFinalizeShaderMaps.Num() > 0)
		{
			UE_LOGF(LogShaders, Warning, "Blocking ProcessAsyncResults for %.1fs, processed %u shader maps, %u being compiled", 
				(float)(FPlatformTime::Seconds() - StartTime),
				NumPendingShaderMaps - PendingFinalizeShaderMaps.Num(), 
				NumCompilingShaderMaps);
		}
	}
	else if (NumPendingShaderMaps - PendingFinalizeShaderMaps.Num() > 0)
	{
		UE_LOGF(LogShaders, Verbose, "Completed %u async shader maps, %u more pending, %u being compiled",
			NumPendingShaderMaps - PendingFinalizeShaderMaps.Num(), 
			PendingFinalizeShaderMaps.Num(),
			NumCompilingShaderMaps);
	}

	UpdateNumRemainingAssets();
}

void FShaderCompilingManager::UpdateNumRemainingAssets()
{
	if (IsInGameThread())
	{
		const int32 NumRemainingAssets = GetNumRemainingAssets();
		if (LastNumRemainingAssets != NumRemainingAssets)
		{
			if (NumRemainingAssets == 0)
			{
				// This is important to at least broadcast once we reach 0 remaining assets
				// even if we don't have any UObject to report because some listener are only 
				// interested to be notified when the number of async compilation reaches 0.
				FAssetCompilingManager::Get().OnAssetPostCompileEvent().Broadcast({});
			}

			LastNumRemainingAssets = NumRemainingAssets;
			Notification->Update(NumRemainingAssets);
		}
	}
}

bool FShaderCompilingManager::IsShaderCompilerWorkerRunning(FProcHandle & WorkerHandle)
{
	return FPlatformProcess::IsProcRunning(WorkerHandle);
}

#if WITH_EDITOR

/* Generates a uniform buffer struct member hlsl declaration using the member's metadata. */
static void GenerateUniformBufferStructMember(FString& Result, const FShaderParametersMetadata::FMember& Member, EShaderPlatform ShaderPlatform)
{
	// Generate the base type name.
	FString TypeName;
	Member.GenerateShaderParameterType(TypeName, ShaderPlatform);

	// Generate array dimension post fix
	FString ArrayDim;
	if (Member.GetNumElements() > 0)
	{
		ArrayDim = FString::Printf(TEXT("[%u]"), Member.GetNumElements());
	}

	Result = FString::Printf(TEXT("%s %s%s"), *TypeName, Member.GetName(), *ArrayDim);
}

/* Generates the instanced stereo hlsl code that's dependent on view uniform declarations. */
void GenerateInstancedStereoCode(FString& Result, EShaderPlatform ShaderPlatform)
{
	// Find the InstancedView uniform buffer struct
	const FShaderParametersMetadata* View = nullptr;
	const FShaderParametersMetadata* InstancedView = nullptr;

	for (TLinkedList<FShaderParametersMetadata*>::TIterator StructIt(FShaderParametersMetadata::GetStructList()); StructIt; StructIt.Next())
	{
		if (StructIt->GetShaderVariableName() == FString("View"))
		{
			View = *StructIt;
		}

		if (StructIt->GetShaderVariableName() == FString("InstancedView"))
		{
			InstancedView = *StructIt;
		}

		if (View && InstancedView)
		{
			break;
		}
	}
	checkSlow(View != nullptr);
	checkSlow(InstancedView != nullptr);

	const TArray<FShaderParametersMetadata::FMember>& StructMembersView = View->GetMembers();
	const TArray<FShaderParametersMetadata::FMember>& StructMembersInstanced = InstancedView->GetMembers();

	static const auto CVarViewHasTileOffsetData = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.ViewHasTileOffsetData"));
	const bool bViewHasTileOffsetData = CVarViewHasTileOffsetData ? (CVarViewHasTileOffsetData->GetValueOnAnyThread() != 0) : false;

	Result = "";
	if (bViewHasTileOffsetData)
	{
		Result +=  "struct ViewStateTileOffsetData\r\n";
		Result += "{\r\n";
		Result += "\tFLWCVector3 WorldCameraOrigin;\r\n";
		Result += "\tFLWCVector3 WorldViewOrigin;\r\n";
		Result += "\tFLWCVector3 PrevWorldCameraOrigin;\r\n";
		Result += "\tFLWCVector3 PrevWorldViewOrigin;\r\n";
		Result += "\tFLWCVector3 PreViewTranslation;\r\n";
		Result += "\tFLWCVector3 PrevPreViewTranslation;\r\n";
		Result += "};\r\n";
	}

	// ViewState definition
	Result +=  "struct ViewState\n";
	Result += "{\n";
	for (int32 MemberIndex = 0; MemberIndex < StructMembersInstanced.Num(); ++MemberIndex)
	{
		FString MemberDecl;
		// ViewState is only supposed to contain InstancedView members however we want their original type and array-length instead of their representation in the instanced array
		// SceneRendererPrimaryViewId for example needs to return 	uint SceneRendererPrimaryViewId; and not uint4 InstancedView_SceneRendererPrimaryViewId[2];
		// and that initial representation is in StructMembersView
		GenerateUniformBufferStructMember(MemberDecl, StructMembersView[MemberIndex], ShaderPlatform);
		Result += FString::Printf(TEXT("\t%s;\n"), *MemberDecl);
	}
	Result += "\tFDFInverseMatrix WorldToClip;\n";
	Result += "\tFDFMatrix ClipToWorld;\n";
	Result += "\tFDFMatrix ScreenToWorld;\n";
	Result += "\tFDFMatrix PrevClipToWorld;\n";
	Result += "\tFDFVector3 WorldCameraOrigin;\n";
	Result += "\tFDFVector3 WorldViewOrigin;\n";
	Result += "\tFDFVector3 PrevWorldCameraOrigin;\n";
	Result += "\tFDFVector3 PrevWorldViewOrigin;\n";
	Result += "\tFDFVector3 PreViewTranslation;\n";
	Result += "\tFDFVector3 PrevPreViewTranslation;\n";

	if (bViewHasTileOffsetData)
	{
		Result += "\tViewStateTileOffsetData TileOffset;\n";
	}

	Result += "};\n";

	Result += "\tvoid FinalizeViewState(inout ViewState InOutView);\n";

	// GetPrimaryView definition
	Result += "ViewState GetPrimaryView()\n";
	Result += "{\n";
	Result += "\tViewState Result;\n";
	for (int32 MemberIndex = 0; MemberIndex < StructMembersInstanced.Num(); ++MemberIndex)
	{
		const FShaderParametersMetadata::FMember& Member = StructMembersView[MemberIndex];
		Result += FString::Printf(TEXT("\tResult.%s = View.%s;\n"), Member.GetName(), Member.GetName());
	}
	Result += "\tFinalizeViewState(Result);\n";
	Result += "\treturn Result;\n";
	Result += "}\n";

	// GetInstancedView definition
	Result += "#if INSTANCED_STEREO || MOBILE_MULTI_VIEW\n";
	Result += "ViewState GetInstancedView(uint ViewIndex)\n";
	Result += "{\n";
	Result += "\tViewState Result;\n";
	for (int32 MemberIndex = 0; MemberIndex < StructMembersInstanced.Num(); ++MemberIndex)
	{
		const FShaderParametersMetadata::FMember& ViewMember = StructMembersView[MemberIndex];
		const FShaderParametersMetadata::FMember& InstancedViewMember = StructMembersInstanced[MemberIndex];

		FString ViewMemberTypeName;
		ViewMember.GenerateShaderParameterType(ViewMemberTypeName, ShaderPlatform);

		// this code avoids an assumption that instanced buffer only supports 2 views, to be future-proof
		if (ViewMember.GetNumElements() >= 1 && (InstancedViewMember.GetNumElements() >= 2 * ViewMember.GetNumElements()))
		{
			// if View has an array (even 1-sized) for this index, and InstancedView has Nx (N>=2) the element count -> per-view array
			// Result.TranslucencyLightingVolumeMin[0] = (float4) InstancedView_TranslucencyLightingVolumeMin[ViewIndex * 2 + 0];
			checkf((InstancedViewMember.GetNumElements() % ViewMember.GetNumElements()) == 0, TEXT("Per-view arrays are expected to be stored in an array that is an exact multiple of the original array."));
			for (uint32 ElementIndex = 0; ElementIndex < ViewMember.GetNumElements(); ElementIndex++)
			{
				Result += FString::Printf(
					TEXT("\tResult.%s[%u] = (%s) InstancedView.%s[ViewIndex * %u + %u];\n"),
					ViewMember.GetName(),
					ElementIndex,
					*ViewMemberTypeName,
					InstancedViewMember.GetName(), 
					ViewMember.GetNumElements(), 
					ElementIndex);
			}
		}
		else if (InstancedViewMember.GetNumElements() > 1 && ViewMember.GetNumElements() == 0)
		{
			// if View has a scalar field for this index, and InstancedView has an array with >1 elements -> per-view scalar
			// 	Result.TranslatedWorldToClip = (float4x4) InstancedView_TranslatedWorldToClip[ViewIndex];
			Result += FString::Printf(
				TEXT("\tResult.%s = (%s) InstancedView.%s[ViewIndex];\n"),
				ViewMember.GetName(), 
				*ViewMemberTypeName, 
				InstancedViewMember.GetName());
		}
		else if (InstancedViewMember.GetNumElements() == ViewMember.GetNumElements())
		{
			// if View has the same number of elements for this index as InstancedView, it's backed by a view-dependent array, assume a view-independent field
			// 	Result.TemporalAAParams = InstancedView_TemporalAAParams;
			Result += FString::Printf(
				TEXT("\tResult.%s = InstancedView.%s;\n"),
				ViewMember.GetName(), 
				InstancedViewMember.GetName());
		}
		else
		{
			// something unexpected, better crash now rather than generate wrong shader code and poison DDC 
			UE_LOGF(
				LogShaderCompilers, 
				Fatal,
				"Don't know how to copy View buffers' field %ls (NumElements=%d) from InstancedView field %ls (NumElements=%d)",
				ViewMember.GetName(), 
				ViewMember.GetNumElements(), 
				InstancedViewMember.GetName(), 
				InstancedViewMember.GetNumElements());
		}
	}
	Result += "\tFinalizeViewState(Result);\n";
	Result += "\treturn Result;\n";
	Result += "}\n";
	Result += "#endif\n";
}

/**
 * Basic validation of virtual shader file paths. This used to also require 'VirtualShaderFilePath' to include "/Generated/",
 * which is no longer desired to allow compiling transient code that acts as a proxy for any other shader.
 */
void ValidateShaderFilePath(const FString& VirtualShaderFilePath, const FString& VirtualSourceFilePath)
{
	check(CheckVirtualShaderFilePath(VirtualShaderFilePath));

	checkf(VirtualShaderFilePath == VirtualSourceFilePath || FPaths::GetExtension(VirtualShaderFilePath) == TEXT("ush"),
		TEXT("Incorrect virtual shader path extension for generated file '%s': Generated file must either be the "
				"USF to compile, or a USH file to be included."),
		*VirtualShaderFilePath);
}

/** Lock for the storage of instanced stereo code. */
FCriticalSection GCachedGeneratedInstancedStereoCodeLock;

/** Storage for instanced stereo code so it is not generated every time we compile a shader. */
TMap<EShaderPlatform, FThreadSafeSharedAnsiStringPtr> GCachedGeneratedInstancedStereoCode;

void GlobalBeginCompileShader(
	const FString& DebugGroupName,
	const FVertexFactoryType* VFType,
	const FShaderType* ShaderType,
	const FShaderPipelineType* ShaderPipelineType,
	int32 PermutationId,
	const TCHAR* SourceFilename,
	const TCHAR* FunctionName,
	FShaderTarget Target,
	FShaderCompilerInput& Input,
	bool bAllowDevelopmentShaderCompile,
	const FString& DebugDescription,
	const FString& DebugExtension
)
{
	GlobalBeginCompileShader(
		DebugGroupName,
		VFType,
		ShaderType,
		ShaderPipelineType,
		PermutationId,
		SourceFilename,
		FunctionName,
		Target,
		Input,
		bAllowDevelopmentShaderCompile,
		*DebugDescription,
		*DebugExtension
		);
}

namespace
{
	bool ShaderFrequencyNeedsInstancedStereoMods(const FShaderType* ShaderType)
	{
		return !(IsRayTracingShaderFrequency(ShaderType->GetFrequency()));
	}
}

static bool IsSubstrateSupportForShaderPipeline(const FShaderCompilerInput& Input)
{
	// If a material requires geometry shaders, ensure the target platform supports geometry shaders. 
	// For instance Substrate requires HLSL2021 which can be cross-compiled, but cross-compilation 
	// toolchain might not support geometry shaders.
	bool bPipelineContainsGeometryShader = false;
	Input.Environment.GetCompileArgument(TEXT("PIPELINE_CONTAINS_GEOMETRYSHADER"), bPipelineContainsGeometryShader);
	const bool bCanRHICompileHlsl2021GeometryShaders = RHISupportsGeometryShaders((EShaderPlatform)Input.Target.Platform);
	const bool bIsSubstrateSupportedForPipeline = !bPipelineContainsGeometryShader || bCanRHICompileHlsl2021GeometryShaders;
	return bIsSubstrateSupportedForPipeline;
}

void GlobalBeginCompileShader(
	const FString& DebugGroupName,
	const FVertexFactoryType* VFType,
	const FShaderType* ShaderType,
	const FShaderPipelineType* ShaderPipelineType,
	int32 PermutationId,
	const TCHAR* SourceFilename,
	const TCHAR* FunctionName,
	FShaderTarget Target,
	FShaderCompilerInput& Input,
	bool bAllowDevelopmentShaderCompile,
	const TCHAR* DebugDescription,
	const TCHAR* DebugExtension
	)
{
	LLM_SCOPE_BYTAG(ShaderCompiler);

	TRACE_CPUPROFILER_EVENT_SCOPE(GlobalBeginCompileShader);
	COOK_STAT(ShaderCompilerCookStats::GlobalBeginCompileShaderCalls.fetch_add(1, std::memory_order_relaxed));
	COOK_STAT(FScopedDurationAtomicTimer DurationTimer(ShaderCompilerCookStats::GlobalBeginCompileShaderTimeSec));

	const EShaderPlatform ShaderPlatform = EShaderPlatform(Target.Platform);
	const FName ShaderFormatName = LegacyShaderPlatformToShaderFormat(ShaderPlatform);

	ITargetPlatform* TargetPlatform = GetTargetPlatformManager()->FindTargetPlatformWithSupport(TEXT("ShaderFormat"), ShaderFormatName);

	FShaderCompileUtilities::GenerateBrdfHeaders(ShaderPlatform);

	// NOTE:  Input.bCompilingForShaderPipeline is initialized by the constructor for single versus pipeline jobs, do not initialize again here!

	Input.Target = Target;
	Input.ShaderPlatformName = FDataDrivenShaderPlatformInfo::GetName(ShaderPlatform);
	Input.ShaderFormat = ShaderFormatName;
	Input.PreferredShaderCodeFormat = FDataDrivenShaderPlatformInfo::GetPreferredShaderCodeFormat(ShaderPlatform);
	Input.SupportedHardwareMask = TargetPlatform ? TargetPlatform->GetSupportedHardwareMask() : 0;
	Input.CompressionFormat = GetShaderCompressionFormat();
	GetShaderCompressionOodleSettings(Input.OodleCompressor, Input.OodleLevel);
	Input.VirtualSourceFilePath = SourceFilename;
	Input.EntryPointName = FunctionName;
	Input.DumpDebugInfoRootPath = GShaderCompilingManager->GetAbsoluteShaderDebugInfoDirectory() / Input.ShaderPlatformName.ToString();
	Input.DebugInfoFlags = GShaderCompilingManager->GetDumpShaderDebugInfoFlags();
	// asset material name or "Global"
	Input.DebugGroupName = DebugGroupName;
	Input.DebugDescription = DebugDescription;
	Input.DebugExtension = DebugExtension;
	Input.RootParametersStructure = ShaderType->GetRootParametersMetadata();
	Input.ShaderName = ShaderType->GetName();

	Input.bIncludeUsedOutputs = false;
	Input.bBindlessEnabled = UE::ShaderCompiler::ShouldCompileWithBindlessEnabled(ShaderPlatform, Input);

	if (UseMaterialVSUniformBuffer())
	{
		Input.VSDisallowedUniformBufferNames.Add(TEXT("Material"));
		Input.NonVSDisallowedUniformBufferNames.Add(TEXT("MaterialVS"));
	}
	else
	{
		// Make sure we don't try to include the MaterialVS by mistake
		Input.VSDisallowedUniformBufferNames.Add(TEXT("MaterialVS"));
		Input.NonVSDisallowedUniformBufferNames.Add(TEXT("MaterialVS"));
	}

	if (GDumpShaderDebugInfoBindless)
	{
		TStringBuilder<32> Builder;
		Builder.Append("Bindless");

		switch (FShaderPlatformConfig::GetBindlessConfiguration(ShaderPlatform))
		{
		default:
		case ERHIBindlessConfiguration::Disabled:   Builder.Append(TEXT("Off")); break;
		case ERHIBindlessConfiguration::RayTracing: Builder.Append(TEXT("RT")); break;
		case ERHIBindlessConfiguration::Minimal:    Builder.Append(TEXT("Min")); break;
		case ERHIBindlessConfiguration::All:        Builder.Append(TEXT("All")); break;
		}

		Input.DebugGroupName = Builder.ToString() / Input.DebugGroupName;
	}

	// Verify FShaderCompilerInput's file paths are consistent. 
	#if DO_CHECK
		check(CheckVirtualShaderFilePath(Input.VirtualSourceFilePath));

		checkf(FPaths::GetExtension(Input.VirtualSourceFilePath) == TEXT("usf"),
			TEXT("Incorrect virtual shader path extension for shader file to compile '%s': Only .usf files should be "
				 "compiled. .ush file are meant to be included only."),
			*Input.VirtualSourceFilePath);

		for (const auto& Entry : Input.Environment.IncludeVirtualPathToContentsMap)
		{
			ValidateShaderFilePath(Entry.Key, Input.VirtualSourceFilePath);
		}

		for (const auto& Entry : Input.Environment.IncludeVirtualPathToSharedContentsMap)
		{
			ValidateShaderFilePath(Entry.Key, Input.VirtualSourceFilePath);
		}
	#endif

	if (ShaderPipelineType)
	{
		Input.DebugGroupName = Input.DebugGroupName / ShaderPipelineType->GetName();
	}
	
	if (VFType)
	{
		FString VFName = VFType->GetName();
		if (GDumpShaderDebugInfoShort)
		{
			// Shorten vertex factory name
			if (VFName[0] == TCHAR('F') || VFName[0] == TCHAR('T'))
			{
				VFName.RemoveAt(0);
			}
			VFName.ReplaceInline(TEXT("VertexFactory"), TEXT("VF"));
			VFName.ReplaceInline(TEXT("GPUSkinAPEXCloth"), TEXT("APEX"));
			VFName.ReplaceInline(TEXT("true"), TEXT("_1"));
			VFName.ReplaceInline(TEXT("false"), TEXT("_0"));
		}
		Input.DebugGroupName = Input.DebugGroupName / VFName;
	}
	
	{
		FString ShaderTypeName = ShaderType->GetName();
		if (GDumpShaderDebugInfoShort)
		{
			// Shorten known types
			if (ShaderTypeName[0] == TCHAR('F') || ShaderTypeName[0] == TCHAR('T'))
			{
				ShaderTypeName.RemoveAt(0);
			}
		}
		Input.DebugGroupName = Input.DebugGroupName / ShaderTypeName / FString::Printf(TEXT("%i"), PermutationId);
		
		if (GDumpShaderDebugInfoShort)
		{
			Input.DebugGroupName.ReplaceInline(TEXT("BasePass"), TEXT("BP"));
			Input.DebugGroupName.ReplaceInline(TEXT("ForForward"), TEXT("Fwd"));
			Input.DebugGroupName.ReplaceInline(TEXT("Shadow"), TEXT("Shdw"));
			Input.DebugGroupName.ReplaceInline(TEXT("LightMap"), TEXT("LM"));
			Input.DebugGroupName.ReplaceInline(TEXT("EHeightFogFeature==E_"), TEXT(""));
			Input.DebugGroupName.ReplaceInline(TEXT("Capsule"), TEXT("Caps"));
			Input.DebugGroupName.ReplaceInline(TEXT("Movable"), TEXT("Mov"));
			Input.DebugGroupName.ReplaceInline(TEXT("Culling"), TEXT("Cull"));
			Input.DebugGroupName.ReplaceInline(TEXT("Atmospheric"), TEXT("Atm"));
			Input.DebugGroupName.ReplaceInline(TEXT("Atmosphere"), TEXT("Atm"));
			Input.DebugGroupName.ReplaceInline(TEXT("Exponential"), TEXT("Exp"));
			Input.DebugGroupName.ReplaceInline(TEXT("Ambient"), TEXT("Amb"));
			Input.DebugGroupName.ReplaceInline(TEXT("Perspective"), TEXT("Persp"));
			Input.DebugGroupName.ReplaceInline(TEXT("Occlusion"), TEXT("Occ"));
			Input.DebugGroupName.ReplaceInline(TEXT("Position"), TEXT("Pos"));
			Input.DebugGroupName.ReplaceInline(TEXT("Skylight"), TEXT("Sky"));
			Input.DebugGroupName.ReplaceInline(TEXT("LightingPolicy"), TEXT("LP"));
			Input.DebugGroupName.ReplaceInline(TEXT("TranslucentLighting"), TEXT("TranslLight"));
			Input.DebugGroupName.ReplaceInline(TEXT("Translucency"), TEXT("Transl"));
			Input.DebugGroupName.ReplaceInline(TEXT("DistanceField"), TEXT("DistFiel"));
			Input.DebugGroupName.ReplaceInline(TEXT("Indirect"), TEXT("Ind"));
			Input.DebugGroupName.ReplaceInline(TEXT("Cached"), TEXT("Cach"));
			Input.DebugGroupName.ReplaceInline(TEXT("Inject"), TEXT("Inj"));
			Input.DebugGroupName.ReplaceInline(TEXT("Visualization"), TEXT("Viz"));
			Input.DebugGroupName.ReplaceInline(TEXT("Instanced"), TEXT("Inst"));
			Input.DebugGroupName.ReplaceInline(TEXT("Evaluate"), TEXT("Eval"));
			Input.DebugGroupName.ReplaceInline(TEXT("Landscape"), TEXT("Land"));
			Input.DebugGroupName.ReplaceInline(TEXT("Dynamic"), TEXT("Dyn"));
			Input.DebugGroupName.ReplaceInline(TEXT("Vertex"), TEXT("Vtx"));
			Input.DebugGroupName.ReplaceInline(TEXT("Output"), TEXT("Out"));
			Input.DebugGroupName.ReplaceInline(TEXT("Directional"), TEXT("Dir"));
			Input.DebugGroupName.ReplaceInline(TEXT("Irradiance"), TEXT("Irr"));
			Input.DebugGroupName.ReplaceInline(TEXT("Deferred"), TEXT("Def"));
			Input.DebugGroupName.ReplaceInline(TEXT("true"), TEXT("_1"));
			Input.DebugGroupName.ReplaceInline(TEXT("false"), TEXT("_0"));
			Input.DebugGroupName.ReplaceInline(TEXT("PROPAGATE_AO"), TEXT("AO"));
			Input.DebugGroupName.ReplaceInline(TEXT("PROPAGATE_SECONDARY_OCCLUSION"), TEXT("SEC_OCC"));
			Input.DebugGroupName.ReplaceInline(TEXT("PROPAGATE_MULTIPLE_BOUNCES"), TEXT("MULT_BOUNC"));
			Input.DebugGroupName.ReplaceInline(TEXT("LocalLights"), TEXT("LL"));
			Input.DebugGroupName.ReplaceInline(TEXT("PostProcess"), TEXT("Post"));
			Input.DebugGroupName.ReplaceInline(TEXT("AntiAliasing"), TEXT("AA"));
			Input.DebugGroupName.ReplaceInline(TEXT("Mobile"), TEXT("Mob"));
			Input.DebugGroupName.ReplaceInline(TEXT("Linear"), TEXT("Lin"));
			Input.DebugGroupName.ReplaceInline(TEXT("INT32_MAX"), TEXT("IMAX"));
			Input.DebugGroupName.ReplaceInline(TEXT("Policy"), TEXT("Pol"));
			Input.DebugGroupName.ReplaceInline(TEXT("EAtmRenderFlag==E_"), TEXT(""));
		}
	}

	// Setup the debug info path if requested, or if this is a global shader and shader development mode is enabled
	Input.DumpDebugInfoPath.Empty();
	if (GShaderCompilingManager->GetDumpShaderDebugInfo() == FShaderCompilingManager::EDumpShaderDebugInfo::Always)
	{
		Input.DumpDebugInfoPath = GShaderCompilingManager->CreateShaderDebugInfoPath(Input);
	}

	// Add the appropriate definitions for the shader frequency.
	{
		SET_SHADER_DEFINE(Input.Environment, PIXELSHADER,				Target.Frequency == SF_Pixel);
		SET_SHADER_DEFINE(Input.Environment, VERTEXSHADER,				Target.Frequency == SF_Vertex);
		SET_SHADER_DEFINE(Input.Environment, MESHSHADER,				Target.Frequency == SF_Mesh);
		SET_SHADER_DEFINE(Input.Environment, AMPLIFICATIONSHADER,		Target.Frequency == SF_Amplification);
		SET_SHADER_DEFINE(Input.Environment, GEOMETRYSHADER,			Target.Frequency == SF_Geometry);
		SET_SHADER_DEFINE(Input.Environment, COMPUTESHADER,				Target.Frequency == SF_Compute);
		SET_SHADER_DEFINE(Input.Environment, RAYCALLABLESHADER,			Target.Frequency == SF_RayCallable);
		SET_SHADER_DEFINE(Input.Environment, RAYHITGROUPSHADER,			Target.Frequency == SF_RayHitGroup);
		SET_SHADER_DEFINE(Input.Environment, RAYGENSHADER,				Target.Frequency == SF_RayGen);
		SET_SHADER_DEFINE(Input.Environment, RAYMISSSHADER,				Target.Frequency == SF_RayMiss);
		SET_SHADER_DEFINE(Input.Environment, WORKGRAPHROOTSHADER,		Target.Frequency == SF_WorkGraphRoot);
		SET_SHADER_DEFINE(Input.Environment, WORKGRAPHCOMPUTESHADER,	Target.Frequency == SF_WorkGraphComputeNode);
	}

	SET_SHADER_DEFINE(Input.Environment, FORWARD_SHADING_FORCES_SKYLIGHT_CUBEMAPS_BLENDING, ForwardShadingForcesSkyLightCubemapBlending(ShaderPlatform) ? 1 : 0);

	SET_SHADER_DEFINE(Input.Environment, SUPPORTS_SKYLIGHTCAPTURE_AFFECTS_HEIGHTFOGINSCATTERING, SupportsSkyLightCaptureAffectsHeightFogInScattering(ShaderPlatform) ? 1 : 0);

	// Enables HLSL 2021
	uint32 EnablesHLSL2021ByDefault = FDataDrivenShaderPlatformInfo::GetEnablesHLSL2021ByDefault(EShaderPlatform(Target.Platform));
	bool bInlineRayTracing = Input.Environment.CompilerFlags.Contains(CFLAG_InlineRayTracing);
	if((EnablesHLSL2021ByDefault == uint32(1) && DebugGroupName == TEXT("Global")) ||
		EnablesHLSL2021ByDefault == uint32(2) ||
		Target.Frequency == SF_RayGen ||			// We want to make sure that function overloads follow c++ rules for FRayDesc.
		Target.Frequency == SF_RayHitGroup ||
		bInlineRayTracing)
	{
		Input.Environment.CompilerFlags.Add(CFLAG_HLSL2021);
	}

	// #defines get stripped out by the preprocessor without this. We can override with this
	SET_SHADER_DEFINE(Input.Environment, COMPILER_DEFINE, TEXT("#define"));

	if (FSceneInterface::GetShadingPath(GetMaxSupportedFeatureLevel(ShaderPlatform)) == EShadingPath::Deferred)
	{
		SET_SHADER_DEFINE(Input.Environment, SHADING_PATH_DEFERRED, 1);
	}

	const bool bUsingMobileRenderer = FSceneInterface::GetShadingPath(GetMaxSupportedFeatureLevel(ShaderPlatform)) == EShadingPath::Mobile;
	if (bUsingMobileRenderer)
	{
		SET_SHADER_DEFINE_AND_COMPILE_ARGUMENT(Input.Environment, SHADING_PATH_MOBILE, true);
		
		const bool bMobileDeferredShading = IsMobileDeferredShadingEnabled((EShaderPlatform)Target.Platform);
		SET_SHADER_DEFINE_AND_COMPILE_ARGUMENT(Input.Environment, MOBILE_DEFERRED_SHADING, bMobileDeferredShading);

		const bool bAllowFramebufferFetch = MobileAllowFramebufferFetch((EShaderPlatform)Target.Platform);
		SET_SHADER_DEFINE(Input.Environment, ALLOW_FRAMEBUFFER_FETCH, bAllowFramebufferFetch);
	
		if (bMobileDeferredShading)
		{
			const bool bGLESDeferredShading = (Target.Platform == SP_OPENGL_ES3_1_ANDROID && bAllowFramebufferFetch);
			SET_SHADER_DEFINE_AND_COMPILE_ARGUMENT(Input.Environment, USE_GLES_FBF_DEFERRED, bGLESDeferredShading);
			SET_SHADER_DEFINE(Input.Environment, MOBILE_EXTENDED_GBUFFER, MobileUsesExtendedGBuffer((EShaderPlatform)Target.Platform) ? 1 : 0);
		}
		else
		{
			static const auto CVarEnableIESProfilesMobileForward = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.Mobile.Forward.EnableIESProfiles"));
			const int32 IESProfilesEnabled = CVarEnableIESProfilesMobileForward ? CVarEnableIESProfilesMobileForward->GetValueOnAnyThread() : 0;
			SET_SHADER_DEFINE(Input.Environment, USE_IES_PROFILE, IESProfilesEnabled);
		}

		SET_SHADER_DEFINE(Input.Environment, USE_SCENE_DEPTH_AUX, MobileRequiresSceneDepthAux(ShaderPlatform) ? 1 : 0);

		static FShaderPlatformCachedIniValue<bool> EnableCullBeforeFetchIniValue(TEXT("r.CullBeforeFetch"));
		if (EnableCullBeforeFetchIniValue.Get((EShaderPlatform)Target.Platform) == 1)
		{
			Input.Environment.CompilerFlags.Add(CFLAG_CullBeforeFetch);
		}

		static FShaderPlatformCachedIniValue<bool> EnableWarpCullingIniValue(TEXT("r.WarpCulling"));
		if (EnableWarpCullingIniValue.Get((EShaderPlatform)Target.Platform) == 1)
		{
			Input.Environment.CompilerFlags.Add(CFLAG_WarpCulling);
		}
	}

	if (RenderRectLightsAsSpotLights(GetMaxSupportedFeatureLevel(ShaderPlatform)))
	{
		SET_SHADER_DEFINE(Input.Environment, RECT_LIGHT_AS_SPOTLIGHT, 1);
	}

	static FShaderPlatformCachedIniValue<bool> ForceSpirvDebugInfoCVar(TEXT("r.ShaderCompiler.ForceSpirvDebugInfo"));
	if (ForceSpirvDebugInfoCVar.Get((EShaderPlatform)Target.GetPlatform()))
	{
		Input.Environment.CompilerFlags.Add(CFLAG_ForceSpirvDebugInfo);
	}

	if (ShaderPlatform == SP_VULKAN_ES3_1_ANDROID || ShaderPlatform == SP_VULKAN_SM5_ANDROID)
	{
		bool bIsStripReflect = true;
		GConfig->GetBool(TEXT("/Script/AndroidRuntimeSettings.AndroidRuntimeSettings"), TEXT("bStripShaderReflection"), bIsStripReflect, GEngineIni);
		if (!bIsStripReflect)
		{
			Input.Environment.SetCompileArgument(TEXT("STRIP_REFLECT_ANDROID"), false);
		}
	}

	static const auto CVarViewHasTileOffsetData = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.ViewHasTileOffsetData"));
	const bool bViewHasTileOffsetData = CVarViewHasTileOffsetData->GetValueOnAnyThread() != 0;
	SET_SHADER_DEFINE_AND_COMPILE_ARGUMENT(Input.Environment, VIEW_HAS_TILEOFFSET_DATA, bViewHasTileOffsetData);

	SET_SHADER_DEFINE_AND_COMPILE_ARGUMENT(Input.Environment, PRIMITIVE_HAS_TILEOFFSET_DATA, GPrimitiveHasTileOffsetData);

	// Set VR definitions
	if (ShaderFrequencyNeedsInstancedStereoMods(ShaderType))
	{
		const UE::StereoRenderUtils::FStereoShaderAspects Aspects(ShaderPlatform);

		SET_SHADER_DEFINE_AND_COMPILE_ARGUMENT(Input.Environment, INSTANCED_STEREO, Aspects.IsInstancedStereoEnabled());
		SET_SHADER_DEFINE_AND_COMPILE_ARGUMENT(Input.Environment, MOBILE_MULTI_VIEW, Aspects.IsMobileMultiViewEnabled());

		// Throw a warning if we are silently disabling ISR due to missing platform support (but don't have MMV enabled).
		static const auto CVarInstancedStereo = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("vr.InstancedStereo"));
		const bool bIsInstancedStereoEnabledInSettings = CVarInstancedStereo ? (CVarInstancedStereo->GetValueOnAnyThread() != 0) : false;
		static const auto CVarMultiview = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("vr.MobileMultiView"));
		const bool bIsMultiviewEnabledInSettings = CVarMultiview ? (CVarMultiview->GetValueOnAnyThread() != 0) : false;
		bool bWarningIssued = false;
		// warn if ISR was enabled in settings, but aspects show that it's not enabled AND we don't use Mobile MultiView as an alternative
		if (bIsInstancedStereoEnabledInSettings && !Aspects.IsInstancedStereoEnabled() && !(bIsMultiviewEnabledInSettings && Aspects.IsMobileMultiViewEnabled()) && !GShaderCompilingManager->AreWarningsSuppressed(ShaderPlatform))
		{
			UE_LOGF(LogShaderCompilers, Warning, "Instanced stereo rendering is not supported for %ls shader platform.", *ShaderFormatName.ToString());
			bWarningIssued = true;
		}
		// Warn if MMV was enabled in settings, but aspects show that it's not enabled AND we don't use Instanced Stereo as an alternative
		if (bIsMultiviewEnabledInSettings && !Aspects.IsMobileMultiViewEnabled() && !(bIsInstancedStereoEnabledInSettings && Aspects.IsInstancedStereoEnabled()) && !GShaderCompilingManager->AreWarningsSuppressed(ShaderPlatform))
		{
			UE_LOGF(LogShaderCompilers, Warning, "Multiview rendering is not supported for %ls shader platform.", *ShaderFormatName.ToString());
			bWarningIssued = true;
		}
		if (bWarningIssued)
		{
			GShaderCompilingManager->SuppressWarnings(ShaderPlatform);
		}
	}
	else
	{
		SET_SHADER_DEFINE_AND_COMPILE_ARGUMENT(Input.Environment, INSTANCED_STEREO, false);
		SET_SHADER_DEFINE_AND_COMPILE_ARGUMENT(Input.Environment, MOBILE_MULTI_VIEW, false);
	}

	// Reserve space in maps to prevent reallocation and rehashing in AddUniformBufferIncludesToEnvironment -- plus one at the end is for GeneratedInstancedStereo.ush
	const int32 UniformBufferReserveNum = Input.Environment.UniformBufferMap.Num() + ShaderType->GetReferencedUniformBuffers().Num() + (VFType ? VFType->GetReferencedUniformBuffers().Num() : 0) + 1;
	Input.Environment.UniformBufferMap.Reserve(UniformBufferReserveNum);
	Input.Environment.IncludeVirtualPathToSharedContentsMap.Reserve(UniformBufferReserveNum);

	ShaderType->AddUniformBufferIncludesToEnvironment(Input.Environment, ShaderPlatform);

	if (VFType)
	{
		VFType->AddUniformBufferIncludesToEnvironment(Input.Environment, ShaderPlatform);
	}

	// Add generated instanced stereo code (this code also generates ViewState, so needed not just for ISR)
	{
		// this function may be called on multiple threads, so protect the storage
		FScopeLock GeneratedInstancedCodeLock(&GCachedGeneratedInstancedStereoCodeLock);

		FThreadSafeSharedAnsiStringPtr* Existing = GCachedGeneratedInstancedStereoCode.Find(ShaderPlatform);
		FThreadSafeSharedAnsiStringPtr CachedCodePtr = Existing ? *Existing : nullptr;
		if (!CachedCodePtr.IsValid())
		{
			FString CachedCode;
			GenerateInstancedStereoCode(CachedCode, ShaderPlatform);

			CachedCodePtr = MakeShareable(new TArray<ANSICHAR>());
			ShaderConvertAndStripComments(CachedCode, *CachedCodePtr);

			GCachedGeneratedInstancedStereoCode.Add(ShaderPlatform, CachedCodePtr);
		}

		Input.Environment.IncludeVirtualPathToSharedContentsMap.Add(TEXT("/Engine/Generated/GeneratedInstancedStereo.ush"), CachedCodePtr);
	}

	{
		// Check if the compile environment explicitly wants to force optimization
		const bool bForceOptimization = Input.Environment.CompilerFlags.Contains(CFLAG_ForceOptimization);

		if (!bForceOptimization && !ShouldOptimizeShaders(ShaderFormatName))
		{
			Input.Environment.CompilerFlags.Add(CFLAG_Debug);
		}
	}

	// Extra data (names, etc)
	if (ShouldEnableExtraShaderData(ShaderFormatName))
	{
		Input.Environment.CompilerFlags.Add(CFLAG_ExtraShaderData);
	}

	// Symbols
	if (ShouldGenerateShaderSymbols(ShaderFormatName))
	{
		Input.Environment.CompilerFlags.Add(CFLAG_GenerateSymbols);
	}
	if (ShouldGenerateShaderSymbolsInfo(ShaderFormatName))
	{
		Input.Environment.CompilerFlags.Add(CFLAG_GenerateSymbolsInfo);
	}

	// Are symbols based on source or results
	if (ShouldAllowUniqueShaderSymbols(ShaderFormatName))
	{
		Input.Environment.CompilerFlags.Add(CFLAG_AllowUniqueSymbols);
	}

	if (CVarShaderFastMath.GetValueOnAnyThread() == 0)
	{
		Input.Environment.CompilerFlags.Add(CFLAG_NoFastMath);
	}
    
	// At this point only apply the cvar r.Mobile.FloatPrecisionMode value to global shaders
	if (bUsingMobileRenderer && (ShaderType == nullptr || ShaderType->GetGlobalShaderType() != nullptr))
    {
        static IConsoleVariable* CVar = IConsoleManager::Get().FindConsoleVariable(TEXT("r.Mobile.FloatPrecisionMode"));
        Input.Environment.FullPrecisionInPS |= CVar ? (CVar->GetInt() == EMobileFloatPrecisionMode::Full) : false;
    }
	
	{
		int32 FlowControl = CVarShaderFlowControl.GetValueOnAnyThread();
		switch (FlowControl)
		{
			case 2:
				Input.Environment.CompilerFlags.Add(CFLAG_AvoidFlowControl);
				break;
			case 1:
				Input.Environment.CompilerFlags.Add(CFLAG_PreferFlowControl);
				break;
			case 0:
				// Fallback to nothing...
			default:
				break;
		}
	}

	{
		static const auto CVar = IConsoleManager::Get().FindConsoleVariable(TEXT("r.Shaders.Validation"));
		if (CVar && CVar->GetInt() == 0)
		{
			Input.Environment.CompilerFlags.Add(CFLAG_SkipValidation);
		}
	}

	{
		SET_SHADER_DEFINE(Input.Environment, DO_CHECK, GSShaderCheckLevel > 0 ? 1 : 0);
		SET_SHADER_DEFINE(Input.Environment, DO_GUARD_SLOW, GSShaderCheckLevel > 1 ? 1 : 0);
	}

	{
		static FShaderPlatformCachedIniValue<int32> CVarWarningsAsErrorsPerPlatform(TEXT("r.Shaders.WarningsAsErrors"));
		const int WarnLevel = CVarWarningsAsErrorsPerPlatform.Get(ShaderPlatform);
		if ((WarnLevel == 1 && ShaderType->GetTypeForDynamicCast() == FShaderType::EShaderTypeForDynamicCast::Global) || WarnLevel > 1)
		{
			Input.Environment.CompilerFlags.Add(CFLAG_WarningsAsErrors);
		}
	}

	if (UseRemoveUnsedInterpolators((EShaderPlatform)Target.Platform) && !IsOpenGLPlatform((EShaderPlatform)Target.Platform))
	{
		Input.Environment.CompilerFlags.Add(CFLAG_ForceRemoveUnusedInterpolators);
	}
	
	if (IsD3DPlatform((EShaderPlatform)Target.Platform) && IsPCPlatform((EShaderPlatform)Target.Platform))
	{


		if (CVarD3DCheckedForTypedUAVs.GetValueOnAnyThread() == 0)
		{
			Input.Environment.CompilerFlags.Add(CFLAG_AllowTypedUAVLoads);
		}

		{
			static const auto CVar = IConsoleManager::Get().FindConsoleVariable(TEXT("r.D3D.CheckedForTypedUAVs"));
			if (CVar && CVar->GetInt() == 0)
			{
				Input.Environment.CompilerFlags.Add(CFLAG_AllowTypedUAVLoads);
			}
		}
	}

	if (IsMetalPlatform((EShaderPlatform)Target.Platform))
	{
		if (CVarShaderZeroInitialise.GetValueOnAnyThread() != 0)
		{
			Input.Environment.CompilerFlags.Add(CFLAG_ZeroInitialise);
		}

		if (CVarShaderBoundsChecking.GetValueOnAnyThread() != 0)
		{
			Input.Environment.CompilerFlags.Add(CFLAG_BoundsChecking);
		}

		// Decide wether to use Metal Native Shader Library or not
		bool bArchive = false;
		GConfig->GetBool(TEXT("/Script/UnrealEd.ProjectPackagingSettings"), TEXT("bSharedMaterialNativeLibraries"), bArchive, GGameIni);
		if (bArchive)
		{
			Input.Environment.CompilerFlags.Add(CFLAG_Archive);
		}
		
		{
			uint32 ShaderVersion = RHIGetMetalShaderLanguageVersion(EShaderPlatform(Target.Platform));
			Input.Environment.SetCompileArgument(TEXT("SHADER_LANGUAGE_VERSION"), ShaderVersion);
			
			bool bAllowFastIntrinsics = false;
			bool bForceFloats = false;
			bool bEnableMathOptimisations = true;
			bool bMetalOptimizeForSize = false;
			bool bAllowsRuntimeShaderCompiling = false;
			bool bStripIndividualMetalLibs = false;
			
			const bool bIsMac = IsPCPlatform(EShaderPlatform(Target.Platform));
			const TCHAR* RuntimeSettingsClassName = bIsMac ? FMacPlatformProperties<false, false, false>::GetRuntimeSettingsClassName() : FIOSPlatformProperties::GetRuntimeSettingsClassName();

			GConfig->GetBool(RuntimeSettingsClassName, TEXT("UseFastIntrinsics"), bAllowFastIntrinsics, GEngineIni);
			GConfig->GetBool(RuntimeSettingsClassName, TEXT("EnableMathOptimisations"), bEnableMathOptimisations, GEngineIni);
			GConfig->GetBool(RuntimeSettingsClassName, TEXT("ForceFloats"), bForceFloats, GEngineIni);
			GConfig->GetBool(RuntimeSettingsClassName, TEXT("MetalOptimizeForSize"), bMetalOptimizeForSize, GEngineIni);
			GConfig->GetBool(RuntimeSettingsClassName, TEXT("AllowsRuntimeShaderCompiling"), bAllowsRuntimeShaderCompiling, GEngineIni);
			GConfig->GetBool(RuntimeSettingsClassName, TEXT("StripIndividualMetalLibs"), bStripIndividualMetalLibs, GEngineIni);
			
			if (bIsMac)
			{
				// No half precision support on MacOS at the moment
				bForceFloats = true;
			}
			else
			{
				// Force no development shaders on iOS
				bAllowDevelopmentShaderCompile = false;
			}
            
            Input.Environment.FullPrecisionInPS |= bForceFloats;
            
			Input.Environment.SetCompileArgument(TEXT("METAL_USE_FAST_INTRINSICS"), bAllowFastIntrinsics);
            Input.Environment.SetCompileArgument(TEXT("METAL_OPTIMIZE_FOR_SIZE"), bMetalOptimizeForSize);
			
			static const auto CVarSymbols = IConsoleManager::Get().FindConsoleVariable(TEXT("r.Shaders.Symbols"));
			static const auto CVarExtraData = IConsoleManager::Get().FindConsoleVariable(TEXT("r.Shaders.ExtraData"));
			
			if (CVarSymbols && CVarExtraData)
			{
				bStripIndividualMetalLibs = bStripIndividualMetalLibs && CVarSymbols->GetInt() == 0 && CVarExtraData->GetInt() == 0;
			}
			Input.Environment.SetCompileArgument(TEXT("METAL_STRIP_INDIVIDUAL_METALLIBS"), bStripIndividualMetalLibs);
			
			
			// Check whether we can compile metal shaders to bytecode - avoids poisoning the DDC
			static ITargetPlatformManagerModule& TPM = GetTargetPlatformManagerRef();
			const IShaderFormat* Compiler = TPM.FindShaderFormat(ShaderFormatName);
			const bool bCanCompileOfflineMetalShaders = Compiler && Compiler->CanCompileBinaryShaders();
			if (!bCanCompileOfflineMetalShaders)
			{
				Input.Environment.CompilerFlags.Add(CFLAG_Debug);
			}
			Input.Environment.SetCompileArgument(TEXT("METAL_ALLOW_RUNTIME_SHADER_COMPILING"), !bCanCompileOfflineMetalShaders || bAllowsRuntimeShaderCompiling);
			
			// Same as console-variable above, but that's global and this is per-platform, per-project
			if (!bEnableMathOptimisations)
			{
				Input.Environment.CompilerFlags.Add(CFLAG_NoFastMath);
			}
		}
	}

	if (IsAndroidPlatform(EShaderPlatform(Target.Platform)))
	{
		// Force no development shaders on Android platforms
		bAllowDevelopmentShaderCompile = false;
	}

	// Mobile emulation should be defined when a PC platform is using a mobile renderer (limited to feature level ES3_1)...  eg SP_PCD3D_ES3_1,SP_VULKAN_PCES3_1,SP_METAL_ES3_1
	if (IsSimulatedPlatform(EShaderPlatform(Target.Platform)) && bAllowDevelopmentShaderCompile)
	{
		SET_SHADER_DEFINE(Input.Environment, MOBILE_EMULATION, 1);
	}

	// Add compiler flag CFLAG_ForceDXC if DXC is enabled
	const bool bHlslVersion2021 = Input.Environment.CompilerFlags.Contains(CFLAG_HLSL2021);
	const bool bIsDxcEnabled = IsDxcEnabledForPlatform((EShaderPlatform)Target.Platform, bHlslVersion2021);
	SET_SHADER_DEFINE(Input.Environment, COMPILER_DXC, bIsDxcEnabled);
	if (bIsDxcEnabled)
	{
		Input.Environment.CompilerFlags.Add(CFLAG_ForceDXC);
	}

	bool bIsMobilePlatform = IsMobilePlatform((EShaderPlatform)Target.Platform);

	if (bIsMobilePlatform)
	{
		if (IsUsingEmulatedUniformBuffers((EShaderPlatform)Target.Platform))
		{
			Input.Environment.CompilerFlags.Add(CFLAG_UseEmulatedUB);
		}
	}

	if (Input.Environment.CompilerFlags.Contains(CFLAG_HLSL2021))
	{
		SET_SHADER_DEFINE(Input.Environment, COMPILER_SUPPORTS_HLSL2021, 1);
	}

	{
		static const auto CVar = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.ClearCoatNormal"));
		bool bCCBN = CVar && CVar->GetValueOnAnyThread() != 0 && (!bIsMobilePlatform || !IsMobileDeferredShadingEnabled((EShaderPlatform)Target.Platform) || MobileUsesExtendedGBuffer((EShaderPlatform)Target.Platform));
		SET_SHADER_DEFINE(Input.Environment, CLEAR_COAT_BOTTOM_NORMAL, bCCBN ? 1 : 0);
	}

	{
		static const auto CVar = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.IrisNormal"));
		SET_SHADER_DEFINE(Input.Environment, IRIS_NORMAL, CVar ? (CVar->GetValueOnAnyThread() != 0) : 0);
	}

	{
		static const auto CVar = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("Compat.UseDXT5NormalMaps"));
		SET_SHADER_DEFINE(Input.Environment, DXT5_NORMALMAPS, CVar ? (CVar->GetValueOnAnyThread() != 0) : 0);
	}

	if (bAllowDevelopmentShaderCompile)
	{
		static const auto CVar = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.CompileShadersForDevelopment"));
		SET_SHADER_DEFINE(Input.Environment, COMPILE_SHADERS_FOR_DEVELOPMENT, CVar ? (CVar->GetValueOnAnyThread() != 0) : 0);
	}

	{
		SET_SHADER_DEFINE(Input.Environment, ALLOW_STATIC_LIGHTING, IsStaticLightingAllowed() ? 1 : 0);
	}

	{
		// Allow GBuffer containing a velocity target to be overridden at a higher level with GBUFFER_LAYOUT
		bool bUsingBasePassVelocity = IsUsingBasePassVelocity((EShaderPlatform)Target.Platform);
		SET_SHADER_DEFINE(Input.Environment, USES_BASE_PASS_VELOCITY, bUsingBasePassVelocity ? 1 : 0);

		bool bGBufferHasVelocity = bUsingBasePassVelocity;
		if (!bGBufferHasVelocity)
		{
			const EGBufferLayout Layout = FShaderCompileUtilities::FetchGBufferLayout(Input.Environment);
			bGBufferHasVelocity |= (Layout == GBL_ForceVelocity);
		}
		SET_SHADER_DEFINE(Input.Environment, GBUFFER_HAS_VELOCITY, bGBufferHasVelocity ? 1 : 0);
	}

	{
		static const auto CVar = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.GBufferDiffuseSampleOcclusion"));
		SET_SHADER_DEFINE(Input.Environment, GBUFFER_HAS_DIFFUSE_SAMPLE_OCCLUSION, CVar ? (CVar->GetValueOnAnyThread() != 0) : 1);
	}

	{
		SET_SHADER_DEFINE(Input.Environment, SELECTIVE_BASEPASS_OUTPUTS, IsUsingSelectiveBasePassOutputs((EShaderPlatform)Target.Platform) ? 1 : 0);
	}

	{
		SET_SHADER_DEFINE(Input.Environment, USE_DBUFFER, IsUsingDBuffers((EShaderPlatform)Target.Platform) ? 1 : 0);
	}

	{
		static IConsoleVariable* CVar = IConsoleManager::Get().FindConsoleVariable(TEXT("r.AllowGlobalClipPlane"));
		SET_SHADER_DEFINE(Input.Environment, PROJECT_ALLOW_GLOBAL_CLIP_PLANE, CVar ? (CVar->GetInt() != 0) : 0);
	}

	{
		const bool bSupportsClipDistance = FDataDrivenShaderPlatformInfo::GetSupportsClipDistance((EShaderPlatform)Target.Platform);
		SET_SHADER_DEFINE(Input.Environment, PLATFORM_SUPPORTS_CLIP_DISTANCE, bSupportsClipDistance ? 1u : 0u);
	}

	{
		const bool bSupportsVertexShaderSRVs = FDataDrivenShaderPlatformInfo::GetSupportsVertexShaderSRVs((EShaderPlatform)Target.Platform);
		SET_SHADER_DEFINE(Input.Environment, PLATFORM_SUPPORTS_VERTEX_SHADER_SRVS, bSupportsVertexShaderSRVs ? 1u : 0u);
	}

	{
		const bool bSupportsVertexShaderUAVs = FDataDrivenShaderPlatformInfo::GetSupportsVertexShaderUAVs((EShaderPlatform)Target.Platform) != ERHIFeatureSupport::Unsupported;
		SET_SHADER_DEFINE(Input.Environment, PLATFORM_SUPPORTS_VERTEX_SHADER_UAVS, bSupportsVertexShaderUAVs ? 1u : 0u);
	}

	{
		const uint32 MaxSamplers = FDataDrivenShaderPlatformInfo::GetMaxSamplers((EShaderPlatform)Target.Platform);
		SET_SHADER_DEFINE_AND_COMPILE_ARGUMENT(Input.Environment, PLATFORM_MAX_SAMPLERS, MaxSamplers);
	}

	{
		const bool bSupportsIndependentSamplers = FDataDrivenShaderPlatformInfo::GetSupportsIndependentSamplers((EShaderPlatform)Target.Platform);
		SET_SHADER_DEFINE(Input.Environment, SUPPORTS_INDEPENDENT_SAMPLERS, bSupportsIndependentSamplers ? 1 : 0);
	}

	bool bForwardShading = false;
	{
		if (bIsMobilePlatform)
		{
			bForwardShading = !IsMobileDeferredShadingEnabled((EShaderPlatform)Target.Platform);
		}
		else if (TargetPlatform)
		{
			bForwardShading = TargetPlatform->UsesForwardShading();
		}
		else
		{
			static IConsoleVariable* CVarForwardShading = IConsoleManager::Get().FindConsoleVariable(TEXT("r.ForwardShading"));
			bForwardShading = CVarForwardShading ? (CVarForwardShading->GetInt() != 0) : false;
		}
		SET_SHADER_DEFINE(Input.Environment, FORWARD_SHADING, bForwardShading);
	}

	{
		if (VelocityEncodeDepth((EShaderPlatform)Target.Platform))
		{
			SET_SHADER_DEFINE(Input.Environment, VELOCITY_ENCODE_DEPTH, 1);
			SET_SHADER_DEFINE(Input.Environment, VELOCITY_SUPPORT_TEMPORAL_RESPONSIVENESS, VelocitySupportsTemporalResponsiveness((EShaderPlatform)Target.Platform) ? 1 : 0);
			SET_SHADER_DEFINE(Input.Environment, VELOCITY_SUPPORT_PS_MOTION_VECTOR_WORLD_OFFSET, VelocitySupportsPixelShaderMotionVectorWorldOffset((EShaderPlatform)Target.Platform) ? 1 : 0);
		}	
		else
		{
			SET_SHADER_DEFINE(Input.Environment, VELOCITY_ENCODE_DEPTH, 0);
			SET_SHADER_DEFINE(Input.Environment, VELOCITY_SUPPORT_TEMPORAL_RESPONSIVENESS, 0);
			SET_SHADER_DEFINE(Input.Environment, VELOCITY_SUPPORT_PS_MOTION_VECTOR_WORLD_OFFSET, 0);
		}
	}

	{
		if (MaskedInEarlyPass((EShaderPlatform)Target.Platform))
		{
			SET_SHADER_DEFINE(Input.Environment, EARLY_Z_PASS_ONLY_MATERIAL_MASKING, 1);
		}
		else
		{
			SET_SHADER_DEFINE(Input.Environment, EARLY_Z_PASS_ONLY_MATERIAL_MASKING, 0);
		}
	}

	{
		static IConsoleVariable* CVar = IConsoleManager::Get().FindConsoleVariable(TEXT("r.VertexFoggingForOpaque"));
		bool bVertexFoggingForOpaque = false;
		if (bForwardShading)
		{
			bVertexFoggingForOpaque = CVar ? (CVar->GetInt() != 0) : 0;
			if (TargetPlatform)
			{
				const int32 PlatformHeightFogMode = TargetPlatform->GetHeightFogModeForOpaque();
				if (PlatformHeightFogMode == 1)
				{
					bVertexFoggingForOpaque = false;
				}
				else if (PlatformHeightFogMode == 2)
				{
					bVertexFoggingForOpaque = true;
				}
			}
		}
		SET_SHADER_DEFINE(Input.Environment, PROJECT_VERTEX_FOGGING_FOR_OPAQUE, bVertexFoggingForOpaque);
	}

	{
		static IConsoleVariable* CVar = IConsoleManager::Get().FindConsoleVariable(TEXT("r.Mobile.DisableVertexFog"));
		SET_SHADER_DEFINE(Input.Environment, PROJECT_MOBILE_DISABLE_VERTEX_FOG, CVar ? (CVar->GetInt() != 0) : 0);
	}

	bool bSupportLocalFogVolumes = false;
	{
		static IConsoleVariable* CVar = IConsoleManager::Get().FindConsoleVariable(TEXT("r.SupportLocalFogVolumes"));
		bSupportLocalFogVolumes = CVar && CVar->GetInt() > 0;
		SET_SHADER_DEFINE(Input.Environment, PROJECT_SUPPORTS_LOCALFOGVOLUME, (bSupportLocalFogVolumes ? 1 : 0));
	}

	{
		static IConsoleVariable* CVar = IConsoleManager::Get().FindConsoleVariable(TEXT("r.LocalFogVolume.ApplyOnTranslucent"));
		const bool bLocalFogVolumesApplyOnTranclucent = CVar && CVar->GetInt() > 0;
		SET_SHADER_DEFINE(Input.Environment, PROJECT_LOCALFOGVOLUME_APPLYONTRANSLUCENT, ((bSupportLocalFogVolumes && bLocalFogVolumesApplyOnTranclucent) ? 1 : 0));
	}

	bool bSupportSkyAtmosphere = false;
	{
		static IConsoleVariable* CVar = IConsoleManager::Get().FindConsoleVariable(TEXT("r.SupportSkyAtmosphere"));
		bSupportSkyAtmosphere = CVar && CVar->GetInt() != 0;
		SET_SHADER_DEFINE(Input.Environment, PROJECT_SUPPORT_SKY_ATMOSPHERE, bSupportSkyAtmosphere ? 1 : 0);
	}

	{
		static IConsoleVariable* CVar = IConsoleManager::Get().FindConsoleVariable(TEXT("r.SupportCloudShadowOnForwardLitTranslucent"));
		const bool bSupportCloudShadowOnForwardLitTranslucent = CVar && CVar->GetInt() > 0;
		SET_SHADER_DEFINE(Input.Environment, SUPPORT_CLOUD_SHADOW_ON_FORWARD_LIT_TRANSLUCENT, bSupportCloudShadowOnForwardLitTranslucent ? 1 : 0);
	}

	{
		static IConsoleVariable *CVar = IConsoleManager::Get().FindConsoleVariable(TEXT("r.Water.SingleLayerWater.SupportCloudShadow"));
		const bool bSupportCloudShadowOnSingleLayerWater = CVar && CVar->GetInt() > 0;
		SET_SHADER_DEFINE(Input.Environment, SUPPORT_CLOUD_SHADOW_ON_SINGLE_LAYER_WATER, bSupportCloudShadowOnSingleLayerWater ? 1 : 0);
	}

	{
		const bool bTranslucentUsesLightRectLights = GetTranslucentUsesLightRectLights();
		SET_SHADER_DEFINE(Input.Environment, SUPPORT_RECTLIGHT_ON_FORWARD_LIT_TRANSLUCENT, bTranslucentUsesLightRectLights ? 1 : 0);
	}

	{
		const bool bTranslucentUsesShadowedLocalLights = GetTranslucentUsesShadowedLocalLights();
		SET_SHADER_DEFINE(Input.Environment, SUPPORT_SHADOWED_LOCAL_LIGHT_ON_FORWARD_LIT_TRANSLUCENT, bTranslucentUsesShadowedLocalLights ? 1 : 0);
	}

	{
		const bool bTranslucentUsesLightIESProfiles = GetTranslucentUsesLightIESProfiles();
		SET_SHADER_DEFINE(Input.Environment, SUPPORT_IESPROFILE_ON_FORWARD_LIT_TRANSLUCENT, bTranslucentUsesLightIESProfiles ? 1 : 0);
	}

	{
		static IConsoleVariable* CVar = IConsoleManager::Get().FindConsoleVariable(TEXT("r.Shadow.Virtual.TranslucentQuality"));
		const bool bHighQualityShadow = CVar && CVar->GetInt() > 0;
		SET_SHADER_DEFINE(Input.Environment, SUPPORT_VSM_FOWARD_QUALITY, bHighQualityShadow ? 1 : 0);
	}

	{
		const bool bUseTriangleStrips = GetHairStrandsUsesTriangleStrips();
		SET_SHADER_DEFINE(Input.Environment, USE_HAIR_TRIANGLE_STRIP, bUseTriangleStrips ? 1 : 0);
	}

	{
		const bool bHasFirstPersonGBufferBit = HasFirstPersonGBufferBit(Target.GetPlatform());
		SET_SHADER_DEFINE(Input.Environment, HAS_FIRST_PERSON_GBUFFER_BIT, bHasFirstPersonGBufferBit ? 1 : 0);
	}

	const bool bSubstrate = Substrate::IsSubstrateEnabled() && IsSubstrateSupportForShaderPipeline(Input);
	{
		SET_SHADER_DEFINE(Input.Environment, SUBSTRATE_ENABLED, bSubstrate ? 1 : 0);

		// "New GBuffer" is the substrate way of packing data. When false the "Legacy Blendable GBuffer" is used (no need to use DBuffer decals).
		SET_SHADER_DEFINE(Input.Environment, SUBSTRATE_GBUFFER_FORMAT, bSubstrate && !Substrate::IsSubstrateBlendableGBufferEnabled(Target.GetPlatform()) ? 1 : 0);

		if (bSubstrate)
		{
			const uint32 SubstrateShadingQuality = FMath::Clamp(Substrate::GetShadingQuality(Target.GetPlatform()), 0u, 1u);
			SET_SHADER_DEFINE(Input.Environment, SUBSTRATE_SHADING_QUALITY, SubstrateShadingQuality);

			const bool bLowQuality = SubstrateShadingQuality == 0;
			SET_SHADER_DEFINE(Input.Environment, USE_ACHROMATIC_BXDF_ENERGY, bLowQuality ? 1u : 0u);

			const uint32 SubstrateNormalQuality = Substrate::GetNormalQuality();
			SET_SHADER_DEFINE(Input.Environment, SUBSTRATE_NORMAL_QUALITY, SubstrateNormalQuality);
			if (SubstrateNormalQuality == 0)
			{
				SET_SHADER_DEFINE(Input.Environment, SUBSTRATE_TOP_LAYER_TYPE, TEXT("uint"));
			}
			else
			{
				SET_SHADER_DEFINE(Input.Environment, SUBSTRATE_TOP_LAYER_TYPE, TEXT("uint2"));
			}

			const uint32 SubstrateUintPerPixel = Substrate::GetBytePerPixel(Target.GetPlatform()) / 4u;
			SET_SHADER_DEFINE(Input.Environment, SUBSTRATE_MATERIAL_NUM_UINTS, SubstrateUintPerPixel);

			// Get the highest closure count accounting for all material quality levels for SUBSTRATE_MATERIAL_CLOSURE_COUNT.
			// This is because it needs to account for the maximum value for all the global shaders being compiled only once. (no permutation per material quality level).
			uint32 SubstrateClosurePerPixel = 1;
			if (!Substrate::IsSubstrateBlendableGBufferEnabled(Target.GetPlatform()))
			{
				for (int8 MaterialQualityLevel = EMaterialQualityLevel::Low; MaterialQualityLevel < EMaterialQualityLevel::Num; ++MaterialQualityLevel)
				{
					SubstrateClosurePerPixel = FMath::Max(SubstrateClosurePerPixel, Substrate::GetClosurePerPixel(Target.GetPlatform(), MaterialQualityLevel));
				}
			}
			SET_SHADER_DEFINE(Input.Environment, SUBSTRATE_MATERIAL_CLOSURE_COUNT, SubstrateClosurePerPixel);

			const bool bSubstrateDBufferPass = Substrate::IsDBufferPassEnabled(Target.GetPlatform());
			SET_SHADER_DEFINE(Input.Environment, SUBSTRATE_USE_DBUFFER_PASS, bSubstrateDBufferPass ? 1 : 0);

			const bool bSubstrateGlints = Substrate::IsGlintEnabled(Target.GetPlatform());
			SET_SHADER_DEFINE(Input.Environment, PLATFORM_ENABLES_SUBSTRATE_GLINTS, bSubstrateGlints ? 1 : 0);

			const bool bSpecularProfileEnabled = Substrate::IsSpecularProfileEnabled(Target.GetPlatform());
			SET_SHADER_DEFINE(Input.Environment, PLATFORM_ENABLES_SUBSTRATE_SPECULAR_PROFILE, bSpecularProfileEnabled ? 1 : 0);
		}
		else
		{
			// Some global uniform buffers reference this type -- so we need to have it defined in all cases
			SET_SHADER_DEFINE(Input.Environment, SUBSTRATE_TOP_LAYER_TYPE, TEXT("uint"));
			SET_SHADER_DEFINE(Input.Environment, SUBSTRATE_MATERIAL_CLOSURE_COUNT, 0);
		}

		const bool bSubstrateOpaqueRoughRefrac = bSubstrate && Substrate::IsOpaqueRoughRefractionEnabled(Target.GetPlatform());
		SET_SHADER_DEFINE(Input.Environment, SUBSTRATE_OPAQUE_ROUGH_REFRACTION_ENABLED, bSubstrateOpaqueRoughRefrac ? 1 : 0);

		const bool bSubstrateAdvDebug = bSubstrate && Substrate::IsAdvancedVisualizationEnabled(Target.GetPlatform());
		SET_SHADER_DEFINE(Input.Environment, SUBSTRATE_ADVANCED_DEBUG_ENABLED, bSubstrateAdvDebug ? 1 : 0);

		const bool IsStochasticLightingEnabled = bSubstrate && Substrate::IsStochasticLightingEnabled(Target.GetPlatform());
		SET_SHADER_DEFINE(Input.Environment, SUBSTRATE_STOCHASTIC_LIGHTING_ENABLED, IsStochasticLightingEnabled ? 1 : 0);

		{
			static IConsoleVariable* CVar = IConsoleManager::Get().FindConsoleVariable(TEXT("r.Substrate.Experimental.ToonReflectionQuantizationEnabled"));
			const bool bExperimentalToonReflectionQuantizationEnabled = bSubstrate && CVar && CVar->GetInt() > 0 ? 1 : 0;
			SET_SHADER_DEFINE(Input.Environment, SUBSTRATE_EXPERIMENTAL_TOON_REFLECTION_QUANTIZATION, bExperimentalToonReflectionQuantizationEnabled ? 1 : 0);
		}

		{
			static IConsoleVariable* CVar = IConsoleManager::Get().FindConsoleVariable(TEXT("r.Substrate.Experimental.ToonUnifiedDiffuse"));
			const bool bExperimentalToonUnifiedDiffuseEnabled = bSubstrate && CVar && CVar->GetInt() > 0 ? 1 : 0;
			SET_SHADER_DEFINE(Input.Environment, SUBSTRATE_EXPERIMENTAL_TOON_UNIFIED_DIFFUSE, bExperimentalToonUnifiedDiffuseEnabled ? 1 : 0);
		}
	}

	{
		static IConsoleVariable* CVar = IConsoleManager::Get().FindConsoleVariable(TEXT("r.Material.RoughDiffuse"));
		const bool bMaterialRoughDiffuse = CVar && CVar->GetInt() != 0;
		const bool bSubstrateRoughDiffuse = Substrate::IsRoughDiffuseEnabled(Target.GetPlatform());
		SET_SHADER_DEFINE(Input.Environment, MATERIAL_ROUGHDIFFUSE, (bSubstrate ? bSubstrateRoughDiffuse : bMaterialRoughDiffuse) ? 1 : 0);
	}

	{
		const bool bLumenSupported = DoesProjectSupportLumenGI((EShaderPlatform)Target.Platform);
		SET_SHADER_DEFINE(Input.Environment, PROJECT_SUPPORTS_LUMEN, bLumenSupported ? 1 : 0);
	}

	{
		const bool bMegaLightsSupported = ShouldCompileMegaLightsShaders((EShaderPlatform)Target.Platform);
		SET_SHADER_DEFINE(Input.Environment, PROJECT_SUPPORTS_MEGALIGHTS, bMegaLightsSupported ? 1 : 0);
	}

	{
		const bool bSupportOIT = FDataDrivenShaderPlatformInfo::GetSupportsOIT(EShaderPlatform(Target.Platform));
		static IConsoleVariable* CVar = IConsoleManager::Get().FindConsoleVariable(TEXT("r.OIT.SortedPixels"));
		const bool bOIT = CVar && CVar->GetInt() != 0;
		SET_SHADER_DEFINE(Input.Environment, PROJECT_OIT, (bSupportOIT && bOIT) ? 1 : 0);
	}

	{
		static IConsoleVariable* CVar = IConsoleManager::Get().FindConsoleVariable(TEXT("r.Material.EnergyConservation"));
		const bool bMaterialEnergyConservation = CVar && CVar->GetInt() != 0;
		SET_SHADER_DEFINE(Input.Environment, LEGACY_MATERIAL_ENERGYCONSERVATION, bMaterialEnergyConservation ? 1 : 0);
	}

	{
		static IConsoleVariable* CVar = IConsoleManager::Get().FindConsoleVariable(TEXT("r.SupportSkyAtmosphereAffectsHeightFog"));
		SET_SHADER_DEFINE(Input.Environment, PROJECT_SUPPORT_SKY_ATMOSPHERE_AFFECTS_HEIGHFOG, (CVar && bSupportSkyAtmosphere) ? (CVar->GetInt() != 0) : 0);
	}

	{
		SET_SHADER_DEFINE(Input.Environment, PROJECT_EXPFOG_MATCHES_VFOG, DoesProjectSupportExpFogMatchesVolumetricFog() ? 1 : 0);
	}

	{
		static IConsoleVariable* CVar = IConsoleManager::Get().FindConsoleVariable(TEXT("r.Deferred.SupportPrimitiveAlphaHoldout"));
		const bool bDeferredSupportPrimitiveAlphaHoldout = CVar->GetBool();

		SET_SHADER_DEFINE(Input.Environment, SUPPORT_PRIMITIVE_ALPHA_HOLDOUT, bDeferredSupportPrimitiveAlphaHoldout ? 1 : 0);
	}

	if (TargetPlatform && 
		TargetPlatform->SupportsFeature(ETargetPlatformFeatures::NormalmapLAEncodingMode))
	{
		SET_SHADER_DEFINE(Input.Environment, LA_NORMALMAPS, 1);
	}

	SET_SHADER_DEFINE(Input.Environment, COLORED_LIGHT_FUNCTION_ATLAS, GetLightFunctionAtlasFormat() > 0 ? 1 : 0);

	// USING_VERTEX_SHADER_LAYER is only intended as alternative for geometry shaders, e.g. for Mac/IOS (-Preview) platform. Don't use it when geometry shaders are available.
	SET_SHADER_DEFINE(Input.Environment, USING_VERTEX_SHADER_LAYER, !RHISupportsGeometryShaders(EShaderPlatform(Target.Platform)) && RHISupportsVertexShaderLayer(EShaderPlatform(Target.Platform)) ? 1 : 0);

	SET_SHADER_DEFINE(Input.Environment, PLATFORM_SUPPORTS_SHADER_ROOT_CONSTANTS, RHISupportsShaderRootConstants(EShaderPlatform(Target.Platform)) ? 1 : 0);
	SET_SHADER_DEFINE(Input.Environment, PLATFORM_SUPPORTS_SHADER_BUNDLE_DISPATCH, RHISupportsShaderBundleDispatch(EShaderPlatform(Target.Platform)) ? 1 : 0);
	SET_SHADER_DEFINE(Input.Environment, PLATFORM_SUPPORTS_RENDERTARGET_WRITE_MASK, RHISupportsRenderTargetWriteMask(EShaderPlatform(Target.Platform)) ? 1 : 0);
	SET_SHADER_DEFINE(Input.Environment, PLATFORM_SUPPORTS_PER_PIXEL_DBUFFER_MASK, FDataDrivenShaderPlatformInfo::GetSupportsPerPixelDBufferMask(EShaderPlatform(Target.Platform)) ? 1 : 0);
	SET_SHADER_DEFINE(Input.Environment, PLATFORM_SUPPORTS_DISTANCE_FIELDS, DoesPlatformSupportDistanceFields(EShaderPlatform(Target.Platform)) ? 1 : 0);
	SET_SHADER_DEFINE(Input.Environment, PLATFORM_SUPPORTS_MESH_SHADERS_TIER0, RHISupportsMeshShadersTier0(EShaderPlatform(Target.Platform)) ? 1 : 0);
	SET_SHADER_DEFINE(Input.Environment, PLATFORM_SUPPORTS_MESH_SHADERS_TIER1, RHISupportsMeshShadersTier1(EShaderPlatform(Target.Platform)) ? 1 : 0);
	SET_SHADER_DEFINE(Input.Environment, PLATFORM_ALLOW_SCENE_DATA_COMPRESSED_TRANSFORMS, FDataDrivenShaderPlatformInfo::GetSupportSceneDataCompressedTransforms(EShaderPlatform(Target.Platform)) ? 1 : 0);
	SET_SHADER_DEFINE(Input.Environment, PLATFORM_SUPPORTS_BUFFER_LOAD_TYPE_CONVERSION, RHISupportsBufferLoadTypeConversion(ShaderPlatform) ? 1 : 0);
	SET_SHADER_DEFINE(Input.Environment, PLATFORM_SUPPORTS_UNIFORM_BUFFER_OBJECTS, FDataDrivenShaderPlatformInfo::GetSupportsUniformBufferObjects(EShaderPlatform(Target.Platform)) ? 1 : 0);
	SET_SHADER_DEFINE(Input.Environment, COMPILER_SUPPORTS_BARYCENTRIC_INTRINSICS, FDataDrivenShaderPlatformInfo::GetSupportsBarycentricsIntrinsics(EShaderPlatform(Target.Platform)));
	SET_SHADER_DEFINE(Input.Environment, PLATFORM_SUPPORTS_BARYCENTRICS_SEMANTIC, FDataDrivenShaderPlatformInfo::GetSupportsBarycentricsSemantic(EShaderPlatform(Target.Platform)) != ERHIFeatureSupport::Unsupported);
	SET_SHADER_DEFINE(Input.Environment, PLATFORM_SUPPORTS_BINDLESS, FDataDrivenShaderPlatformInfo::GetSupportsBindless(ShaderPlatform));
	SET_SHADER_DEFINE(Input.Environment, PLATFORM_SUPPORTS_DESCRIPTOR_RANGE, FDataDrivenShaderPlatformInfo::GetSupportsDescriptorRange(ShaderPlatform));
	SET_SHADER_DEFINE(Input.Environment, PLATFORM_SUPPORTS_RAYTRACING_CLUSTER_OPS, FDataDrivenShaderPlatformInfo::GetSupportsRayTracingClusterOps(ShaderPlatform));

	// Differentiate between specialization constants being supported by the platform vs being enabled for a shader.
	SET_SHADER_DEFINE(Input.Environment, PLATFORM_SUPPORTS_SPECIALIZATION_CONSTANTS, FDataDrivenShaderPlatformInfo::GetSupportsSpecializationConstants(ShaderPlatform));
	const bool bEnableSpecializationConstants = Input.Environment.SpecializationConstantValues.Num() > 0;
	SET_SHADER_DEFINE(Input.Environment, ENABLE_SPECIALIZATION_CONSTANTS, bEnableSpecializationConstants ? 1 : 0);

	SET_SHADER_DEFINE(Input.Environment, UE_BINDLESS_ENABLED, Input.bBindlessEnabled);

	if (Input.Environment.ShaderBindingLayout)
	{
 		Input.Environment.CompilerFlags.Add(CFLAG_ShaderBindingLayout);
	}

	if (CVarShadersRemoveDeadCode.GetValueOnAnyThread())
	{
		Input.Environment.CompilerFlags.Add(CFLAG_RemoveDeadCode);
	}

	if (CVarDisableSourceStripping.GetValueOnAnyThread())
	{
		Input.Environment.CompilerFlags.Add(CFLAG_DisableSourceStripping);
	}

	{
		static const auto CVar = IConsoleManager::Get().FindConsoleVariable(TEXT("r.VT.AnisotropicFiltering"));
		SET_SHADER_DEFINE(Input.Environment, VIRTUAL_TEXTURE_ANISOTROPIC_FILTERING, CVar ? (CVar->GetInt() != 0) : 0);
		
		if (bIsMobilePlatform)
		{
			static FShaderPlatformCachedIniValue<bool> CVarVTMobileManualTrilinearFiltering(TEXT("r.VT.Mobile.ManualTrilinearFiltering"));
			SET_SHADER_DEFINE(Input.Environment, VIRTUAL_TEXTURE_MANUAL_TRILINEAR_FILTERING, (CVarVTMobileManualTrilinearFiltering.Get(Target.GetPlatform()) ? 1 : 0));
		}
	}

	if (bIsMobilePlatform)
	{
		const bool bMobileMovableSpotlightShadowsEnabled = IsMobileMovableSpotlightShadowsEnabled(Target.GetPlatform());
		SET_SHADER_DEFINE(Input.Environment, PROJECT_MOBILE_ENABLE_MOVABLE_SPOTLIGHT_SHADOWS, bMobileMovableSpotlightShadowsEnabled ? 1 : 0);
	}

	{
		using namespace UE::Color;
		const bool bWorkingColorSpaceIsSRGB = FColorSpace::GetWorking().IsSRGB();
		SET_SHADER_DEFINE(Input.Environment, WORKING_COLOR_SPACE_IS_SRGB, bWorkingColorSpaceIsSRGB ? 1 : 0);
		
		// We limit matrix definitions below to WORKING_COLOR_SPACE_IS_SRGB == 0.
		if (!bWorkingColorSpaceIsSRGB)
		{
			static constexpr TCHAR MatrixFormat[] = TEXT("float3x3(%0.10f, %0.10f, %0.10f, %0.10f, %0.10f, %0.10f, %0.10f, %0.10f, %0.10f)");
			const FColorSpace& WorkingColorSpace = FColorSpace::GetWorking();

			// Note that we transpose the matrices during print since color matrices are usually pre-multiplied.
			const FMatrix44d& ToXYZ = WorkingColorSpace.GetRgbToXYZ();
			Input.Environment.SetDefine(
				TEXT("WORKING_COLOR_SPACE_RGB_TO_XYZ_MAT"),
				FString::Printf(MatrixFormat,
					ToXYZ.M[0][0], ToXYZ.M[1][0], ToXYZ.M[2][0],
					ToXYZ.M[0][1], ToXYZ.M[1][1], ToXYZ.M[2][1],
					ToXYZ.M[0][2], ToXYZ.M[1][2], ToXYZ.M[2][2]));

			const FMatrix44d& FromXYZ = WorkingColorSpace.GetXYZToRgb();
			Input.Environment.SetDefine(
				TEXT("XYZ_TO_RGB_WORKING_COLOR_SPACE_MAT"),
				FString::Printf(MatrixFormat,
					FromXYZ.M[0][0], FromXYZ.M[1][0], FromXYZ.M[2][0],
					FromXYZ.M[0][1], FromXYZ.M[1][1], FromXYZ.M[2][1],
					FromXYZ.M[0][2], FromXYZ.M[1][2], FromXYZ.M[2][2]));

			const FColorSpaceTransform& FromSRGB = FColorSpaceTransform::GetSRGBToWorkingColorSpace();
			SET_SHADER_DEFINE(Input.Environment,
				SRGB_TO_WORKING_COLOR_SPACE_MAT,
				FString::Printf(MatrixFormat,
					FromSRGB.M[0][0], FromSRGB.M[1][0], FromSRGB.M[2][0],
					FromSRGB.M[0][1], FromSRGB.M[1][1], FromSRGB.M[2][1],
					FromSRGB.M[0][2], FromSRGB.M[1][2], FromSRGB.M[2][2]));
		}

		static const auto CVar = IConsoleManager::Get().FindConsoleVariable(TEXT("r.LegacyLuminanceFactors"));
		const bool bUseLegacyLuminance = CVar && CVar->GetInt() != 0;
		SET_SHADER_DEFINE(Input.Environment, UE_LEGACY_LUMINANCE_FACTORS, bUseLegacyLuminance ? 1 : 0);
	}

	const double TileSize = FLargeWorldRenderScalar::GetTileSize();
	SET_SHADER_DEFINE(Input.Environment, UE_LWC_RENDER_TILE_SIZE, (float)TileSize);
	SET_SHADER_DEFINE(Input.Environment, UE_LWC_RENDER_TILE_SIZE_SQRT, (float)FMath::Sqrt(TileSize));
	SET_SHADER_DEFINE(Input.Environment, UE_LWC_RENDER_TILE_SIZE_RSQRT, (float)FMath::InvSqrt(TileSize));
	SET_SHADER_DEFINE(Input.Environment, UE_LWC_RENDER_TILE_SIZE_RCP, (float)(1.0 / TileSize));
	SET_SHADER_DEFINE(Input.Environment, UE_LWC_RENDER_TILE_SIZE_FMOD_PI, (float)FMath::Fmod(TileSize, UE_DOUBLE_PI));
	SET_SHADER_DEFINE(Input.Environment, UE_LWC_RENDER_TILE_SIZE_FMOD_2PI, (float)FMath::Fmod(TileSize, 2.0 * UE_DOUBLE_PI));

	{
		// Set Nanite Foliage definitions
		const bool bNanitePlatform = DoesRuntimeSupportNanite(ShaderPlatform, false, true);
		SET_SHADER_DEFINE(Input.Environment, NANITE_ASSEMBLY_DATA, bNanitePlatform && NaniteAssembliesSupported());
		SET_SHADER_DEFINE(Input.Environment, NANITE_VOXEL_DATA, bNanitePlatform && NaniteVoxelsSupported());

		// Set Nanite Curve definitions
		SET_SHADER_DEFINE(Input.Environment, NANITE_CURVE_DATA, bNanitePlatform && NaniteCurvesSupported());
	}

	// Add required symbols from the shader binding layout if set
	if (Input.Environment.ShaderBindingLayout)
	{
		Input.Environment.ShaderBindingLayout->AddRequiredSymbols(Input.RequiredSymbols);
	}

	// Allow the target shader format to modify the shader input before we add it as a job
	const IShaderFormat* Format = GetTargetPlatformManagerRef().FindShaderFormat(ShaderFormatName);
	checkf(Format, TEXT("Shader format %s cannot be found"), *ShaderFormatName.ToString());
	Format->ModifyShaderCompilerInput(Input);

	// Allow the GBuffer and other shader defines to cause dependend environment changes, but minimizing the #ifdef magic in the shaders, which
	// is nearly impossible to debug when it goes wrong.
	FShaderCompileUtilities::ApplyDerivedDefines(Input.Environment, Input.SharedEnvironment, (EShaderPlatform)Target.Platform);
}

#endif // WITH_EDITOR

static bool ParseShaderCompilerFlags(const TCHAR* InFlagsString, FShaderCompilerFlags& OutCompilerFlags)
{
	if (!InFlagsString || *InFlagsString == TEXT('\0'))
	{
		return false;
	}

	TStringBuilder<4096> UnknownFlagNameList;

	FString NextFlagArg;
	while (FParse::Token(InFlagsString, NextFlagArg, false, TEXT('+')))
	{
		ECompilerFlags NextFlag = CFLAG_Max;
		if (NextFlagArg.StartsWith(TEXT("CFLAG_")))
		{
			LexFromString(NextFlag, *NextFlagArg.Mid(6));
		}
		else
		{
			LexFromString(NextFlag, *NextFlagArg);
		}

		if (NextFlag != CFLAG_Max)
		{
			OutCompilerFlags.Add(NextFlag);
		}
		else
		{
			UnknownFlagNameList.Appendf(TEXT("%s%s"), UnknownFlagNameList.Len() > 0 ? TEXT(", ") : TEXT(""), *NextFlagArg);
		}
	}

	if (UnknownFlagNameList.Len() > 0)
	{
		UE_LOGF(LogShaderCompilers, Warning, "Unknown shader compiler flags: %ls", *UnknownFlagNameList);
		return false;
	}

	return true;
}

/** Timer class used to report information on the 'recompileshaders' console command. */
class FRecompileShadersTimer
{
public:
	FRecompileShadersTimer(const TCHAR* InInfoStr=TEXT("Test")) :
		InfoStr(InInfoStr),
		bAlreadyStopped(false)
	{
		StartTime = FPlatformTime::Seconds();
	}

	FRecompileShadersTimer(const FString& InInfoStr) :
		InfoStr(InInfoStr),
		bAlreadyStopped(false)
	{
		StartTime = FPlatformTime::Seconds();
	}

	void Stop(bool DisplayLog = true)
	{
		if (!bAlreadyStopped)
		{
			bAlreadyStopped = true;
			EndTime = FPlatformTime::Seconds();
			TimeElapsed = EndTime-StartTime;
			if (DisplayLog)
			{
				UE_LOGF(LogShaderCompilers, Warning, "		[%ls] took [%.4f] s",*InfoStr,TimeElapsed);
			}
		}
	}

	~FRecompileShadersTimer()
	{
		Stop(true);
	}

protected:
	double StartTime,EndTime;
	double TimeElapsed;
	FString InfoStr;
	bool bAlreadyStopped;
};

namespace
{
	void ListAllShaderTypes()
	{
		UE_LOGF(LogShaderCompilers, Display, "ShaderTypeName, Filename, PermutationCount");
		for (TLinkedList<FShaderType*>::TIterator It(FShaderType::GetTypeList()); It; It.Next())
		{
			UE_LOGF(LogShaderCompilers, Display, "%ls, %ls, %d", (*It)->GetName(), (*It)->GetShaderFilename(), (*It)->GetPermutationCount());
		}

		UE_LOGF(LogShaderCompilers, Display, "VertexFactoryTypeName, Filename");
		for (TLinkedList<FVertexFactoryType*>::TIterator It(FVertexFactoryType::GetTypeList()); It; It.Next())
		{
			UE_LOGF(LogShaderCompilers, Display, "%ls, %ls", (*It)->GetName(), (*It)->GetShaderFilename());
		}
	}

	ODSCRecompileCommand ParseRecompileCommandString(const TCHAR* CmdString, TArray<FString>& OutMaterialsToLoad, FString& OutShaderTypesToLoad, FString& OutRequestedMaterialName, FShaderCompilerFlags& OutExtraCompilerFlags)
	{
		FString CmdName = FParse::Token(CmdString, 0);

		ODSCRecompileCommand CommandType = ODSCRecompileCommand::None;
		OutMaterialsToLoad.Empty();

		if( !CmdName.IsEmpty() && FCString::Stricmp(*CmdName,TEXT("Material"))==0 )
		{
			CommandType = ODSCRecompileCommand::Material;

			// tell other side the material to load, by pathname
			FString RequestedMaterialName( FParse::Token( CmdString, 0 ) );
			OutRequestedMaterialName = RequestedMaterialName;
			UMaterialInterface* MatchingMaterial = nullptr;
			for (TObjectIterator<UMaterialInterface> It; It; ++It)
			{
				UMaterial* Material = It->GetMaterial();

				if (Material && Material->GetName() == RequestedMaterialName)
				{
					OutMaterialsToLoad.Add(It->GetPathName());
					MatchingMaterial = Material;
					break;
				}
			}

			// Find all material instances from the requested material and 
			// request a compile for them.
			if (MatchingMaterial)
			{
				for (TObjectIterator<UMaterialInstance> It; It; ++It)
				{
					if (It && It->IsDependent(MatchingMaterial))
					{
						OutMaterialsToLoad.Add(It->GetPathName());
					}
				}
			}
		}
		else if (!CmdName.IsEmpty() && FCString::Stricmp(*CmdName, TEXT("Global")) == 0)
		{
			CommandType = ODSCRecompileCommand::Global;
		}
		else if (!CmdName.IsEmpty() && FCString::Stricmp(*CmdName, TEXT("Changed")) == 0)
		{
			CommandType = ODSCRecompileCommand::Changed;

			// Compile all the shaders that have changed for the materials we have loaded.
			for (TObjectIterator<UMaterialInterface> It; It; ++It)
			{
				OutMaterialsToLoad.Add(It->GetPathName());
			}
		}
		else if (FCString::Stricmp(*CmdName, TEXT("All")) == 0)
		{
			CommandType = ODSCRecompileCommand::Material;

			// tell other side all the materials to load, by pathname
			for (TObjectIterator<UMaterialInterface> It; It; ++It)
			{
				OutMaterialsToLoad.Add(It->GetPathName());
			}
		}
		else if (FCString::Stricmp(*CmdName, TEXT("listtypes")) == 0)
		{
			ListAllShaderTypes();
		}
		else
		{
			CommandType = ODSCRecompileCommand::SingleShader;

			OutShaderTypesToLoad = CmdName;

			// Parse optional extra compiler flags from commandline
			const FString FlagsStr(FParse::Token(CmdString, false));
			ParseShaderCompilerFlags(*FlagsStr, OutExtraCompilerFlags);

			// tell other side which materials to load and compile the single
			// shader for.
			for (TObjectIterator<UMaterialInterface> It; It; ++It)
			{
				OutMaterialsToLoad.Add(It->GetPathName());
			}
		}

		return CommandType;
	}
}

static int32 GODSCMaterialUpdateFlags = 0;
static FAutoConsoleVariableRef CVarODSCMaterialUpdateFlags(
	TEXT("ODSC.MaterialUpdateFlags"),
	GODSCMaterialUpdateFlags,
	TEXT("Changes the material update flags when ODSC receives new shaders and needs to update the materials\n")
	TEXT("0 (default): no additional work\n")
	TEXT("1: Reregister all components while updating the material\n")
	TEXT("2: Sync with the rendering thread after all the calls to RecacheUniformExpressions\n")
	TEXT("4 (legacy): Recreates only the render state for *all* components, including the ones not changed by ODSC\n")
);

void ProcessCookOnTheFlyShaders(bool bReloadGlobalShaders, const TArray<uint8>& MeshMaterialMaps, const TArray<FString>& MaterialsToLoad, const TArray<uint8>& GlobalShaderMap)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(ProcessCookOnTheFlyShaders);
	check(IsInGameThread());

	bool bHasFlushed = false;

	auto DoFlushIfNecessary = [&bHasFlushed]() {
		if (!bHasFlushed )
		{
			// now we need to refresh the RHI resources
			FlushRenderingCommands();
			bHasFlushed = true;
		}
	};

	// reload the global shaders
	if (bReloadGlobalShaders)
	{
		DoFlushIfNecessary();

		// Some platforms rely on global shaders to be created to implement basic RHI functionality
		TGuardValue<int32> Guard(GCreateShadersOnLoad, 1);
		CompileGlobalShaderMap(true);
	}

	// load all the mesh material shaders if any were sent back
	if (MeshMaterialMaps.Num() > 0)
	{
		DoFlushIfNecessary();

		// parse the shaders
		FMemoryReader MemoryReader(MeshMaterialMaps, true);
		FNameAsStringProxyArchive Ar(MemoryReader);

		TArray<UMaterialInterface*> LoadedMaterials;
		FMaterialShaderMap::LoadForRemoteRecompile(Ar, GMaxRHIShaderPlatform, LoadedMaterials);

		// Only update materials if we need to.
		if (LoadedMaterials.Num())
		{
			// need to force material update flag when reloading default material 
			// since it may be used by any primitive in depth/shadow passes
			const bool bIsDefaultMaterial = LoadedMaterials.ContainsByPredicate([](UMaterialInterface* MaterialInterface)
			{
				UMaterial* Material = Cast<UMaterial>(MaterialInterface);
				return Material != nullptr && Material->IsDefaultMaterial();
			});
			const uint32 MaterialUpdateFlags = GODSCMaterialUpdateFlags | (bIsDefaultMaterial ? 1 : 0);

			// this will stop the rendering thread, and reattach components, in the destructor
			FMaterialUpdateContext UpdateContext(MaterialUpdateFlags);

			// gather the shader maps to reattach
			for (UMaterialInterface* Material : LoadedMaterials)
			{
				// ~FMaterialUpdateContext takes care of calling RecacheUniformExpressions on all MaterialInstances, no need to call it twice
				if (Cast<UMaterialInstance>(Material) == nullptr)
				{
					Material->RecacheUniformExpressions(true);
				}

				UpdateContext.AddMaterialInterface(Material);
			}
		}
	}

	// load all the global shaders if any were sent back
	if (GlobalShaderMap.Num() > 0)
	{
		DoFlushIfNecessary();

		// parse the shaders
		FMemoryReader MemoryReader(GlobalShaderMap, true);
		FNameAsStringProxyArchive Ar(MemoryReader);

		LoadGlobalShadersForRemoteRecompile(Ar, GMaxRHIShaderPlatform);
	}
}

/**
* Forces a recompile of the global shaders.
*/
void RecompileGlobalShaders()
{
#if WITH_EDITOR
	if (!FPlatformProperties::RequiresCookedData())
	{
		// Flush pending accesses to the existing global shaders.
		FlushRenderingCommands();

		UMaterialInterface::IterateOverActiveFeatureLevels([&](ERHIFeatureLevel::Type InFeatureLevel) {
			auto ShaderPlatform = GShaderPlatformForFeatureLevel[InFeatureLevel];
			GetGlobalShaderMap(ShaderPlatform)->Empty();
			VerifyGlobalShaders(ShaderPlatform, nullptr, false);
		});

		GShaderCompilingManager->ProcessAsyncResults(false, true);
	}
#endif // WITH_EDITOR
}

void GetOutdatedShaderTypes(TArray<const FShaderType*>& OutdatedShaderTypes, TArray<const FShaderPipelineType*>& OutdatedShaderPipelineTypes, TArray<const FVertexFactoryType*>& OutdatedFactoryTypes)
{
#if WITH_EDITOR
	for (int PlatformIndex = 0; PlatformIndex < SP_NumPlatforms; ++PlatformIndex)
	{
		const FGlobalShaderMap* ShaderMap = GGlobalShaderMap[PlatformIndex];
		if (ShaderMap)
		{
			ShaderMap->GetOutdatedTypes(OutdatedShaderTypes, OutdatedShaderPipelineTypes, OutdatedFactoryTypes);
		}
	}

	FMaterialShaderMap::GetAllOutdatedTypes(OutdatedShaderTypes, OutdatedShaderPipelineTypes, OutdatedFactoryTypes);

	for (int32 TypeIndex = 0; TypeIndex < OutdatedShaderTypes.Num(); TypeIndex++)
	{
		UE_LOGF(LogShaders, Warning, "		Recompiling %ls", OutdatedShaderTypes[TypeIndex]->GetName());
	}
	for (int32 TypeIndex = 0; TypeIndex < OutdatedShaderPipelineTypes.Num(); TypeIndex++)
	{
		UE_LOGF(LogShaders, Warning, "		Recompiling %ls", OutdatedShaderPipelineTypes[TypeIndex]->GetName());
	}
	for (int32 TypeIndex = 0; TypeIndex < OutdatedFactoryTypes.Num(); TypeIndex++)
	{
		UE_LOGF(LogShaders, Warning, "		Recompiling %ls", OutdatedFactoryTypes[TypeIndex]->GetName());
	}
#endif // WITH_EDITOR
}

bool RecompileShaders(const TCHAR* Cmd, FOutputDevice& Ar)
{
	// if this platform can't compile shaders, then we try to send a message to a file/cooker server
	if (FPlatformProperties::RequiresCookedData())
	{
#if WITH_ODSC
		TArray<FString> MaterialsToLoad;
		FString ShaderTypesToLoad;
		FString RequestedMaterialName;
		FShaderCompilerFlags ExtraCompilerFlags;
		ODSCRecompileCommand CommandType = ParseRecompileCommandString(Cmd, MaterialsToLoad, ShaderTypesToLoad, RequestedMaterialName, ExtraCompilerFlags);

		ERHIFeatureLevel::Type TargetFeatureLevel = GetMaxSupportedFeatureLevel(GMaxRHIShaderPlatform);
		const EMaterialQualityLevel::Type ActiveQualityLevel = GetCachedScalabilityCVars().MaterialQualityLevel;
		if (FODSCManager::IsODSCActive())
		{
			GODSCManager->AddThreadedRequest(MaterialsToLoad, ShaderTypesToLoad, GMaxRHIShaderPlatform, TargetFeatureLevel, ActiveQualityLevel, CommandType, RequestedMaterialName, ExtraCompilerFlags);
		}
		else
		{
			UE_LOGF(LogShaders, Error, "Tried to use recompileshaders but ODSC is not active");
		}
#endif
		return true;
	}

#if WITH_EDITOR
	FString FlagStr(FParse::Token(Cmd, 0));
	if (FlagStr.Len() > 0)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(RecompileShaders);
		GWarn->BeginSlowTask( NSLOCTEXT("ShaderCompilingManager", "BeginRecompilingShadersTask", "Recompiling shaders"), true );

		// Flush the shader file cache so that any changes to shader source files will be detected
		FlushShaderFileCache();
		FlushRenderingCommands();

		if (FCString::Stricmp(*FlagStr,TEXT("Changed")) == 0)
		{
			TArray<const FShaderType*> OutdatedShaderTypes;
			TArray<const FVertexFactoryType*> OutdatedFactoryTypes;
			TArray<const FShaderPipelineType*> OutdatedShaderPipelineTypes;
			{
				FRecompileShadersTimer SearchTimer(TEXT("Searching for changed files"));
				GetOutdatedShaderTypes(OutdatedShaderTypes, OutdatedShaderPipelineTypes, OutdatedFactoryTypes);
			}

			if (OutdatedShaderPipelineTypes.Num() > 0 || OutdatedShaderTypes.Num() > 0 || OutdatedFactoryTypes.Num() > 0)
			{
				FRecompileShadersTimer TestTimer(TEXT("RecompileShaders Changed"));

				UpdateReferencedUniformBufferNames(OutdatedShaderTypes, OutdatedFactoryTypes, OutdatedShaderPipelineTypes);

				// Kick off global shader recompiles
				UMaterialInterface::IterateOverActiveFeatureLevels([&](ERHIFeatureLevel::Type InFeatureLevel) {
					auto ShaderPlatform = GShaderPlatformForFeatureLevel[InFeatureLevel];
					BeginRecompileGlobalShaders(OutdatedShaderTypes, OutdatedShaderPipelineTypes, ShaderPlatform);
					// Block on global shader compilation. Do this for each feature level/platform compiled as otherwise global shader compile job IDs collide.
					FinishRecompileGlobalShaders();
				});

				// Kick off material shader recompiles
				UMaterialInterface::IterateOverActiveFeatureLevels([&](ERHIFeatureLevel::Type InFeatureLevel) {
					auto ShaderPlatform = GShaderPlatformForFeatureLevel[InFeatureLevel];
					UMaterial::UpdateMaterialShaders(OutdatedShaderTypes, OutdatedShaderPipelineTypes, OutdatedFactoryTypes, ShaderPlatform);
				});

				GWarn->StatusUpdate(0, 1, NSLOCTEXT("ShaderCompilingManager", "CompilingGlobalShaderStatus", "Compiling global shaders..."));
			}
			else
			{
				UE_LOGF(LogShaderCompilers, Warning, "No Shader changes found.");
			}
		}
		else if (FCString::Stricmp(*FlagStr, TEXT("Global")) == 0)
		{
			FRecompileShadersTimer TestTimer(TEXT("RecompileShaders Global"));
			RecompileGlobalShaders();
		}
		else if (FCString::Stricmp(*FlagStr, TEXT("Material")) == 0)
		{
			FString RequestedMaterialName(FParse::Token(Cmd, 0));
			FRecompileShadersTimer TestTimer(FString::Printf(TEXT("Recompile Material %s"), *RequestedMaterialName));

			ITargetPlatformManagerModule& TPM = GetTargetPlatformManagerRef();
			FString TargetPlatformName(FParse::Token(Cmd, 0));
			const ITargetPlatform* TargetPlatform = nullptr;
			if (TargetPlatformName.Len() > 0)
			{
				TargetPlatform = TPM.FindTargetPlatform(TargetPlatformName);
			}

			bool bMaterialFound = false;
			for( TObjectIterator<UMaterialInterface> It; It; ++It )
			{
				UMaterialInterface* Material = *It;
				if( Material && Material->GetName() == RequestedMaterialName)
				{
					bMaterialFound = true;

					// <Pre/Post>EditChange will force a re-creation of the resource,
					// in turn recompiling the shader.
					if (TargetPlatform)
					{
						Material->BeginCacheForCookedPlatformData(TargetPlatform);
						while (!Material->IsCachedCookedPlatformDataLoaded(TargetPlatform))
						{
							FPlatformProcess::Sleep(0.1f);
							GShaderCompilingManager->ProcessAsyncResults(false, false);
						}
						Material->ClearCachedCookedPlatformData(TargetPlatform);
					}
					else
					{
						Material->PreEditChange(nullptr);
						Material->PostEditChange();
					}

					break;
				}
			}

			if (!bMaterialFound)
			{
				TestTimer.Stop(false);
				UE_LOGF(LogShaderCompilers, Warning, "Couldn't find Material %ls!", *RequestedMaterialName);
			}
		}
		else if (FCString::Stricmp(*FlagStr, TEXT("All")) == 0)
		{
			FRecompileShadersTimer TestTimer(TEXT("RecompileShaders"));
			RecompileGlobalShaders();

			FMaterialUpdateContext UpdateContext(0);
			for( TObjectIterator<UMaterial> It; It; ++It )
			{
				UMaterial* Material = *It;
				if( Material )
				{
					UE_LOGF(LogShaderCompilers, Log, "recompiling [%ls]",*Material->GetFullName());
					UpdateContext.AddMaterial(Material);

					// <Pre/Post>EditChange will force a re-creation of the resource,
					// in turn recompiling the shader.
					Material->PreEditChange(nullptr);
					Material->PostEditChange();
				}
			}
		}
		else if (FCString::Stricmp(*FlagStr, TEXT("listtypes")) == 0)
		{
			ListAllShaderTypes();
		}
		else
		{
			constexpr bool bSearchAsRegexFilter = true;
			TArray<const FShaderType*> ShaderTypes = FShaderType::GetShaderTypesByFilename(*FlagStr, bSearchAsRegexFilter);
			TArray<const FShaderPipelineType*> ShaderPipelineTypes = FShaderPipelineType::GetShaderPipelineTypesByFilename(*FlagStr, bSearchAsRegexFilter);

			if (ShaderTypes.Num() > 0 || ShaderPipelineTypes.Num() > 0)
			{
				FRecompileShadersTimer TestTimer(TEXT("RecompileShaders SingleShader"));
				
				UpdateReferencedUniformBufferNames(ShaderTypes, {}, ShaderPipelineTypes);

				// Parse optional extra compiler flags from commandline
				FShaderCompilerFlags ExtraCompilerFlags;
				const FString FlagsStr(FParse::Token(Cmd, false));
				ParseShaderCompilerFlags(*FlagsStr, ExtraCompilerFlags);

				UMaterialInterface::IterateOverActiveFeatureLevels([&](ERHIFeatureLevel::Type InFeatureLevel) {
					auto ShaderPlatform = GShaderPlatformForFeatureLevel[InFeatureLevel];
					BeginRecompileGlobalShaders(ShaderTypes, ShaderPipelineTypes, ShaderPlatform, nullptr, ExtraCompilerFlags);
					FinishRecompileGlobalShaders();
				});
			}
		}

		GWarn->EndSlowTask();

		return true;
	}

	UE_LOGF(LogShaderCompilers, Warning, "Invalid parameter. \n"
											 "Options are: \n"
											 "    'Changed'             Recompile just the shaders that have source file changes.\n"
											 "    'Global'              Recompile just the global shaders.\n"
											 "    'Material [name]'     Recompile all the shaders for a single material.\n"
											 "    'Listtypes'           List all the shader type and vertex factory type class names and their source file path.  Can be used to find shader file names to be used with `recompileshaders [shaderfilename]`.\n"
											 "    'All'                 Recompile all materials.\n"
											 "    [filename] [flags]    Compile all shaders associated with a specific filename or regular expression (including '*' for any characters). Optionally add CFLAG entries concatenated with '+'.\n");
#endif // WITH_EDITOR

	return true;
}

#if WITH_EDITORONLY_DATA
namespace ShaderCompilerUtil
{
	FOnGlobalShadersCompilation GOnGlobalShdersCompilationDelegate;
}

FOnGlobalShadersCompilation& GetOnGlobalShaderCompilation()
{
	return ShaderCompilerUtil::GOnGlobalShdersCompilationDelegate;
}
#endif // WITH_EDITORONLY_DATA

/**
* Makes sure all global shaders are loaded and/or compiled for the passed in platform.
* Note: if compilation is needed, this only kicks off the compile.
*
* @param	Platform	Platform to verify global shaders for
*/
void VerifyGlobalShaders(EShaderPlatform Platform, const ITargetPlatform* TargetPlatform, bool bLoadedFromCacheFile, const TArray<const FShaderType*>* OutdatedShaderTypes, const TArray<const FShaderPipelineType*>* OutdatedShaderPipelineTypes, const FShaderCompilerFlags& InExtraCompilerFlags)
{
	SCOPED_LOADTIMER(VerifyGlobalShaders);

	check(IsInGameThread());
	check(!FPlatformProperties::IsServerOnly());
	check(GGlobalShaderMap[Platform]);

	UE_LOGF(LogMaterial, Verbose, "Verifying Global Shaders for %ls (%ls)", *LegacyShaderPlatformToShaderFormat(Platform).ToString(), *ShaderCompiler::GetTargetPlatformName(TargetPlatform));

	// Ensure that the global shader map contains all global shader types.
	FGlobalShaderMap* GlobalShaderMap = GetGlobalShaderMap(Platform);
	const bool bEmptyMap = GlobalShaderMap->IsEmpty();
	if (bEmptyMap)
	{
		UE_LOGF(LogShaders, Log, "	Empty global shader map, recompiling all global shaders");
	}

	FPlatformTypeLayoutParameters LayoutParams;
	LayoutParams.InitializeForPlatform(TargetPlatform);
	EShaderPermutationFlags PermutationFlags = GetShaderPermutationFlags(LayoutParams);

	// if the target is the current platform, then we are not cooking for another platform, in which case we want to use
	// the loaded permutation flags that are in the shader map (or the current platform's permutation if it wasn't loaded, 
	// see the FShaderMapBase constructor)
	if (bLoadedFromCacheFile)
	{
		PermutationFlags = GlobalShaderMap->GetFirstSection()->GetPermutationFlags();
	}

	bool bErrorOnMissing = bLoadedFromCacheFile;
	if (FPlatformProperties::RequiresCookedData())
	{
		// We require all shaders to exist on cooked platforms because we can't compile them.
		bErrorOnMissing = true;
	}

#if WITH_EDITOR
	// All jobs, single & pipeline
	TArray<FShaderCommonCompileJobPtr> GlobalShaderJobs;

	// Add the single jobs first
	TMap<TShaderTypePermutation<const FShaderType>, FShaderCompileJob*> SharedShaderJobs;
#endif // WITH_EDITOR

	for (TLinkedList<FShaderType*>::TIterator ShaderTypeIt(FShaderType::GetTypeList()); ShaderTypeIt; ShaderTypeIt.Next())
	{
		FGlobalShaderType* GlobalShaderType = ShaderTypeIt->GetGlobalShaderType();
		if (!GlobalShaderType)
		{
			continue;
		}

		const bool bOutdated = OutdatedShaderTypes && OutdatedShaderTypes->Contains(GlobalShaderType);

		int32 PermutationCountToCompile = 0;
		GlobalShaderType->ForEachCompiledPermutation(Platform, PermutationFlags, [&](int32 PermutationId)
		{
				TShaderRef<FShader> GlobalShader = GlobalShaderMap->GetShader(GlobalShaderType, PermutationId);
				if (bOutdated || !GlobalShader.IsValid())
				{
					if (bErrorOnMissing)
					{
                        if (IsMetalPlatform(GMaxRHIShaderPlatform))
                        {
                            check(IsInGameThread());
                            FMessageDialog::Open(EAppMsgType::Ok, LOCTEXT("NoGlobalShader_Error", "Missing shader permutation. Please make sure cooking was successful and refer to Engine log for details."));
                        }
                            UE_LOGF(LogShaders, Fatal, "Missing global shader %ls's permutation %i, Please make sure cooking was successful.",
                                GlobalShaderType->GetName(), PermutationId);
                    }
					else
					{
#if WITH_EDITOR
						if (OutdatedShaderTypes)
						{
							// Remove old shader, if it exists
							GlobalShaderMap->RemoveShaderTypePermutaion(GlobalShaderType, PermutationId);
						}

						// Compile this global shader type.
						FGlobalShaderTypeCompiler::BeginCompileShader(GlobalShaderType, PermutationId, Platform, PermutationFlags, GlobalShaderJobs);
						//TShaderTypePermutation<const FShaderType> ShaderTypePermutation(GlobalShaderType, PermutationId);
						//check(!SharedShaderJobs.Find(ShaderTypePermutation));
						//SharedShaderJobs.Add(ShaderTypePermutation, Job);
						PermutationCountToCompile++;
#endif // WITH_EDITOR
					}
				}
			return true;
		});

		int32 PermutationCountLimit = 832;	// Nanite culling as of today (2022-01-11) can go up to 832 permutations
		if (Substrate::IsSubstrateEnabled())
		{
			PermutationCountLimit = 1368;	// FDeferredLightPS as of 2025-08-08
		}
		ensureMsgf(
			PermutationCountToCompile <= PermutationCountLimit,
			TEXT("Global shader %s has %i permutations: probably more than it needs."),
			GlobalShaderType->GetName(), PermutationCountToCompile);

		if (!bEmptyMap && PermutationCountToCompile > 0)
		{
			UE_LOGF(LogShaders, Log, "	%ls (%i out of %i)",
				GlobalShaderType->GetName(), PermutationCountToCompile, GlobalShaderType->GetPermutationCount());
		}
	}

	// Now the pipeline jobs; if it's a shareable pipeline, do not add duplicate jobs
	for (TLinkedList<FShaderPipelineType*>::TIterator ShaderPipelineIt(FShaderPipelineType::GetTypeList()); ShaderPipelineIt; ShaderPipelineIt.Next())
	{
		const FShaderPipelineType* Pipeline = *ShaderPipelineIt;
		if (Pipeline->IsGlobalTypePipeline())
		{
			if (FGlobalShaderType::ShouldCompilePipeline(Pipeline, Platform, PermutationFlags)
				&& (!GlobalShaderMap->HasShaderPipeline(Pipeline) || (OutdatedShaderPipelineTypes && OutdatedShaderPipelineTypes->Contains(Pipeline))))
			{
				if (OutdatedShaderPipelineTypes)
				{
					// Remove old pipeline
					GlobalShaderMap->RemoveShaderPipelineType(Pipeline);
				}

				if (bErrorOnMissing)
				{
					UE_LOGF(LogShaders, Fatal, "Missing global shader pipeline %ls, Please make sure cooking was successful.", Pipeline->GetName());
				}
				else
				{
#if WITH_EDITOR
					if (!bEmptyMap)
					{
						UE_LOGF(LogShaders, Log, "	%ls", Pipeline->GetName());
					}

					if (Pipeline->ShouldOptimizeUnusedOutputs(Platform))
					{
						// Make a pipeline job with all the stages
						FGlobalShaderTypeCompiler::BeginCompileShaderPipeline(Platform, PermutationFlags, Pipeline, GlobalShaderJobs);
					}
					else
					{
						// If sharing shaders amongst pipelines, add this pipeline as a dependency of an existing individual job
						for (const FShaderType* ShaderType : Pipeline->GetStages())
						{
							TShaderTypePermutation<const FShaderType> ShaderTypePermutation(ShaderType, kUniqueShaderPermutationId);

							FShaderCompileJob** Job = SharedShaderJobs.Find(ShaderTypePermutation);
							checkf(Job, TEXT("Couldn't find existing shared job for global shader %s on pipeline %s!"), ShaderType->GetName(), Pipeline->GetName());
							auto* SingleJob = (*Job)->GetSingleShaderJob();
							check(SingleJob);
							auto& SharedPipelinesInJob = SingleJob->SharingPipelines.FindOrAdd(nullptr);
							check(!SharedPipelinesInJob.Contains(Pipeline));
							SharedPipelinesInJob.Add(Pipeline);
						}
					}
#endif // WITH_EDITOR
				}
			}
		}
	}

#if WITH_EDITOR
	if (GlobalShaderJobs.Num() > 0)
	{
		if (InExtraCompilerFlags.GetData() != 0)
		{
			for (FShaderCommonCompileJobPtr& Job : GlobalShaderJobs)
			{
				Job->ForEachSingleShaderJob(
					[InExtraCompilerFlags](FShaderCompileJob& SingleJob) -> void
					{
						SingleJob.Input.Environment.CompilerFlags.Append(InExtraCompilerFlags);
					}
				);
			}
		}

		GetOnGlobalShaderCompilation().Broadcast();
		GShaderCompilingManager->SubmitJobs(GlobalShaderJobs, "Globals");

		const bool bAllowAsynchronousGlobalShaderCompiling =
			// OpenGL requires that global shader maps are compiled before attaching
			// primitives to the scene as it must be able to find FNULLPS.
			// TODO_OPENGL: Allow shaders to be compiled asynchronously.
			// Metal also needs this when using RHI thread because it uses TOneColorVS very early in RHIPostInit()
			!IsOpenGLPlatform(GMaxRHIShaderPlatform) && !IsVulkanPlatform(GMaxRHIShaderPlatform) &&
			!IsMetalPlatform(GMaxRHIShaderPlatform) &&
			FDataDrivenShaderPlatformInfo::GetSupportsAsyncPipelineCompilation(GMaxRHIShaderPlatform) &&
			GShaderCompilingManager->AllowAsynchronousShaderCompiling();

		if (!bAllowAsynchronousGlobalShaderCompiling)
		{
			TArray<int32> ShaderMapIds;
			ShaderMapIds.Add(GlobalShaderMapId);

			GShaderCompilingManager->FinishCompilation(TEXT("Global"), ShaderMapIds);
		}
	}
#endif // WITH_EDITOR
}

void VerifyGlobalShaders(EShaderPlatform Platform, bool bLoadedFromCacheFile, const TArray<const FShaderType*>* OutdatedShaderTypes, const TArray<const FShaderPipelineType*>* OutdatedShaderPipelineTypes)
{
	VerifyGlobalShaders(Platform, nullptr, bLoadedFromCacheFile, OutdatedShaderTypes, OutdatedShaderPipelineTypes);
}

// Declare dummy global PSO collector which doesn't do anything during collection but list to compile is setup below
// in optimized tasks so shaders can be created in parallel to offload the game thread during startup (5+ shaders)
static const TCHAR* GlobalComputePSOCollectorName = TEXT("GlobalComputePSOCollector");
FRegisterGlobalPSOCollectorFunction RegisterGlobalComputePSOCollector(nullptr, GlobalComputePSOCollectorName);

void PrecacheComputePipelineStatesForGlobalShaders(ERHIFeatureLevel::Type FeatureLevel, const ITargetPlatform* TargetPlatform)
{
	const FPSOPrecacheSettings& PSOPrecacheSettings = UPSOPrecacheSettingsManager::GetSettings();
	if (PSOPrecacheSettings.GlobalShaderMode == 0)
	{
		return;
	}

	if (!IsPSOShaderPreloadingEnabled() && !(PipelineStateCache::IsPSOPrecachingEnabled() && GRHISupportsPSOPrecaching))
	{
		return;
	}

	TRACE_CPUPROFILER_EVENT_SCOPE(PrecacheComputePipelineStatesForGlobalShaders);

	FPlatformTypeLayoutParameters LayoutParams;
	LayoutParams.InitializeForPlatform(TargetPlatform);
	EShaderPermutationFlags PermutationFlags = GetShaderPermutationFlags(LayoutParams);

	EShaderPlatform ShaderPlatform = GetFeatureLevelShaderPlatform(FeatureLevel);
	FGlobalShaderMap* GlobalShaderMap = GetGlobalShaderMap(ShaderPlatform);
	
	// some RHIs (OpenGL) can only create shaders on the Render thread. Queue the creation instead of doing it here.
	TArray<TShaderRef<FShader>> ComputeShadersToPrecache;
	for (TLinkedList<FShaderType*>::TIterator ShaderTypeIt(FShaderType::GetTypeList()); ShaderTypeIt; ShaderTypeIt.Next())
	{
		FGlobalShaderType* GlobalShaderType = ShaderTypeIt->GetGlobalShaderType();
		if (!GlobalShaderType || (GlobalShaderType->GetFrequency() != SF_Compute && PSOPrecacheSettings.GlobalShaderMode == 1))
		{
			continue;
		}

		int32 ShaderPermutationPerGlobalShaderType = 0;
		for (int32 PermutationId = 0; PermutationId < GlobalShaderType->GetPermutationCount(); PermutationId++)
		{
			if (GlobalShaderType->ShouldCompilePermutation(ShaderPlatform, PermutationId, PermutationFlags) &&
				GlobalShaderType->ShouldPrecachePermutation(ShaderPlatform, PermutationId, PermutationFlags) == EShaderPermutationPrecacheRequest::Precached)
			{
				TShaderRef<FShader> GlobalShader = GlobalShaderMap->GetShader(GlobalShaderType, PermutationId);
				if (GlobalShader.IsValid())
				{
					ComputeShadersToPrecache.Add(GlobalShader);
					ShaderPermutationPerGlobalShaderType++;
				}
			}
		}

		/*
		int32 PermutationCountLimit = 300;
		ensureMsgf(
			ShaderPermutationPerGlobalShaderType < PermutationCountLimit,
			TEXT("Global shader %s has %i permutations to precache: probably more than it needs."),
			GlobalShaderType->GetName(), ShaderPermutationPerGlobalShaderType);
		*/
	}
	
	if (ComputeShadersToPrecache.Num() > 0)
	{
		if (PipelineStateCache::IsPSOPrecachingEnabled())
		{
			UE_LOGF(LogShaders, Display, "Precaching %d global compute shaders", ComputeShadersToPrecache.Num());

			struct FGlobalComputePSOPrecacheData
			{
				FGlobalComputePSOPrecacheData(int Count)
				{
					PSOInitializers.Reserve(Count);
				}

				FPSOPrecacheDataArray PSOInitializers;
				FCriticalSection PSOInitializersCS;
			};
			FGlobalComputePSOPrecacheData* GlobalComputePSOPrecacheData = new FGlobalComputePSOPrecacheData(ComputeShadersToPrecache.Num());			

			auto PrecacheComputePipelineState = [GlobalComputePSOPrecacheData, PSOPrecacheSettings](TShaderRef<FShader>& GlobalShader)
			{
				// PSO precache shaders are not required to all load correctly
				constexpr bool bRequired = false;
				FRHIComputeShader* RHIComputeShader = static_cast<FRHIComputeShader*>(GlobalShader.GetRHIShaderBase(SF_Compute, bRequired));
				if (RHIComputeShader)
				{							
					static const int32 GlobalComputePSOCollectorIndex = FGlobalPSOCollectorManager::GetIndex(GlobalComputePSOCollectorName);

					check(!RHIComputeShader->HasSpecializationConstants() || !GlobalShader.GetSpecializationConstants().IsEmpty());
					FComputePipelineStateInitializer ComputeInitializer(RHIComputeShader, GlobalShader.GetSpecializationConstants());
								
					FPSOPrecacheData PSOPrecacheData;
					PSOPrecacheData.Type = FPSOPrecacheData::EType::Compute;
					PSOPrecacheData.ComputeInitializer = ComputeInitializer;
					PSOPrecacheData.InMemoryMode = (PSOPrecacheSettings.bKeepGlobalComputePSOsInMemory && KeepPrecachedPSOsInMemoryForActiveMaterials()) ? EPSOPrecacheInMemoryMode::Cached : EPSOPrecacheInMemoryMode::Uncached;
					#if PSO_PRECACHING_VALIDATE
					PSOPrecacheData.ResourceName = GlobalShader.GetType()->GetName();
					PSOPrecacheData.PSOCollectorIndex = GlobalComputePSOCollectorIndex;
					PSOPrecacheData.VertexFactoryType = nullptr;
					#endif // PSO_PRECACHING_VALIDATE

					FScopeLock ScopeLock(&GlobalComputePSOPrecacheData->PSOInitializersCS);
					GlobalComputePSOPrecacheData->PSOInitializers.Add(MoveTemp(PSOPrecacheData));
				}
			};

			if (GRHIGlobals.SupportsMultithreadedShaderCreation)
			{
				TArray<UE::Tasks::TTask<void>> Tasks;
				for (TShaderRef<FShader>& GlobalShader : ComputeShadersToPrecache)
				{
					Tasks.Emplace(UE::Tasks::Launch(TEXT("PrecachePSOsForGlobalShaders"),
						[&GlobalShader, PrecacheComputePipelineState]()
						{
							PrecacheComputePipelineState(GlobalShader);
						}
					));
				}
				ENQUEUE_RENDER_COMMAND(WaitOnPSOPrecacheTasks)(
					[ComputeShadersToPrecache = MoveTemp(ComputeShadersToPrecache), Tasks = MoveTemp(Tasks), GlobalComputePSOPrecacheData](FRHICommandListImmediate& RHICmdList)
					{
						UE::Tasks::Wait(Tasks);

						RequestPrecachePSOs(EPSOPrecacheType::Global, GlobalComputePSOPrecacheData->PSOInitializers);
						delete GlobalComputePSOPrecacheData;
					});
			}
			else
			{
				ENQUEUE_RENDER_COMMAND(PrecachePSOsForGlobalShaders)(
					[ComputeShadersToPrecache = MoveTemp(ComputeShadersToPrecache), GlobalComputePSOPrecacheData, PrecacheComputePipelineState](FRHICommandListImmediate& RHICmdList)
					{
						for (TShaderRef<FShader> GlobalShader : ComputeShadersToPrecache)
						{
							PrecacheComputePipelineState(GlobalShader);
						}
						
						RequestPrecachePSOs(EPSOPrecacheType::Global, GlobalComputePSOPrecacheData->PSOInitializers);
						delete GlobalComputePSOPrecacheData;
					});
			}
		}
		else if (IsPSOShaderPreloadingEnabled())
		{
			// Kick off preloading tasks.
			FGraphEventArray Events;
			for (TShaderRef<FShader> GlobalShader : ComputeShadersToPrecache)
			{
				GlobalShader.GetResource()->PreloadShader(GlobalShader->GetResourceIndex(), Events);
			}
		}
	}
		
	// Collect all global graphics PSOs
	FSceneTexturesConfigInitSettings SceneTexturesConfigInitSettings;
	SceneTexturesConfigInitSettings.FeatureLevel = FeatureLevel;

	FSceneTexturesConfig SceneTexturesConfig;
	SceneTexturesConfig.Init(SceneTexturesConfigInitSettings);
	
	FPSOPrecacheDataArray GlobalPSOInitializers;
	GlobalPSOInitializers.Reserve(1024);
	
	for (int32 Index = 0; Index < FGlobalPSOCollectorManager::GetPSOCollectorCount(); ++Index)
	{
		GlobalPSOCollectorFunction CollectFunction = FGlobalPSOCollectorManager::GetCollectFunction(Index);
		if (CollectFunction)
		{
			CollectFunction(SceneTexturesConfig, Index, GlobalPSOInitializers);
		}
	}

	// Set if the global graphics PSOs need to be kept in memory or not
	for (FPSOPrecacheData& PrecacheData : GlobalPSOInitializers)
	{
		PrecacheData.InMemoryMode = (PSOPrecacheSettings.bKeepGlobalGraphicPSOsInMemory && KeepPrecachedPSOsInMemoryForActiveMaterials()) ? EPSOPrecacheInMemoryMode::Cached : EPSOPrecacheInMemoryMode::Uncached;
	}

	RequestPrecachePSOs(EPSOPrecacheType::Global, GlobalPSOInitializers);
}

#include "Misc/PreLoadFile.h"
#include "Serialization/LargeMemoryReader.h"
static FPreLoadFile GGlobalShaderPreLoadFile(*(FString("../../../Engine") / TEXT("GlobalShaderCache-SP_") + FPlatformProperties::IniPlatformName() + TEXT(".bin")));

const ITargetPlatform* GGlobalShaderTargetPlatform[SP_NumPlatforms] = { nullptr };

static FString GGlobalShaderCacheOverrideDirectory;

static FString GetGlobalShaderCacheOverrideFilename(EShaderPlatform Platform)
{
	FString DirectoryPrefix = FPaths::EngineDir() / TEXT("OverrideGlobalShaderCache-");

	if (!GGlobalShaderCacheOverrideDirectory.IsEmpty())
	{
		DirectoryPrefix = GGlobalShaderCacheOverrideDirectory / TEXT("GlobalShaderCache-");
	}

	return DirectoryPrefix + FDataDrivenShaderPlatformInfo::GetName(Platform).ToString() + TEXT(".bin");
}

static FString GetGlobalShaderCacheFilename(EShaderPlatform Platform)
{
	return FString("Engine") / TEXT("GlobalShaderCache-") + FDataDrivenShaderPlatformInfo::GetName(Platform).ToString() + TEXT(".bin");
}

/** Saves the global shader map as a file for the target platform. */
FString SaveGlobalShaderFile(EShaderPlatform Platform, FString SavePath, class ITargetPlatform* TargetPlatform)
{
	FGlobalShaderMap* GlobalShaderMap = GetGlobalShaderMap(Platform);

	// Wait until all global shaders are compiled
	if (GShaderCompilingManager)
	{
		GShaderCompilingManager->ProcessAsyncResults(false, true);
	}

	TArray<uint8> GlobalShaderData;
	{
#if WITH_EDITOR
		TOptional<FArchiveSavePackageDataBuffer> ArchiveSavePackageData;
#endif
		FMemoryWriter MemoryWriter(GlobalShaderData, true);

#if WITH_EDITOR
		if (TargetPlatform != nullptr)
		{
			ArchiveSavePackageData.Emplace(TargetPlatform);
			MemoryWriter.SetSavePackageData(&ArchiveSavePackageData.GetValue());
		}
#endif // WITH_EDITOR

		GlobalShaderMap->SaveToGlobalArchive(MemoryWriter);
	}

	// make the final name
	FString FullPath = SavePath / GetGlobalShaderCacheFilename(Platform);
	if (!FFileHelper::SaveArrayToFile(GlobalShaderData, *FullPath))
	{
		UE_LOGF(LogShaders, Fatal, "Could not save global shader file to '%ls'", *FullPath);
	}

#if WITH_EDITOR
	if (FShaderLibraryCooker::NeedsShaderStableKeys(Platform))
	{
		GlobalShaderMap->SaveShaderStableKeys(Platform);
	}
#endif // WITH_EDITOR
	return FullPath;
}


static inline bool ShouldCacheGlobalShaderTypeName(const FGlobalShaderType* GlobalShaderType, int32 PermutationId, const TCHAR* TypeNameSubstring, EShaderPlatform Platform, EShaderPermutationFlags PermutationFlags)
{
	return GlobalShaderType
		&& (TypeNameSubstring == nullptr || (FPlatformString::Strstr(GlobalShaderType->GetName(), TypeNameSubstring) != nullptr))
		&& GlobalShaderType->ShouldCompilePermutation(Platform, PermutationId, PermutationFlags);
};


bool IsGlobalShaderMapComplete(const TCHAR* TypeNameSubstring, FGlobalShaderMap* GlobalShaderMap, EShaderPlatform Platform, FString* FailureReason = nullptr)
{
	// look at any shadermap in the GlobalShaderMap for the permutation flags, as they will all be the same
	if (GlobalShaderMap)
	{
		const FGlobalShaderMapSection* FirstShaderMap = GlobalShaderMap->GetFirstSection();
		if (FirstShaderMap == nullptr)
		{
			// if we had no sections at all, we know we aren't complete
			return false;
		}
		EShaderPermutationFlags GlobalShaderPermutation = FirstShaderMap->GetPermutationFlags();

		// Check if the individual shaders are complete
		for (TLinkedList<FShaderType*>::TIterator ShaderTypeIt(FShaderType::GetTypeList()); ShaderTypeIt; ShaderTypeIt.Next())
		{
			FGlobalShaderType* GlobalShaderType = ShaderTypeIt->GetGlobalShaderType();
			int32 PermutationCount = GlobalShaderType ? GlobalShaderType->GetPermutationCount() : 1;
			for (int32 PermutationId = 0; PermutationId < PermutationCount; PermutationId++)
			{
				if (ShouldCacheGlobalShaderTypeName(GlobalShaderType, PermutationId, TypeNameSubstring, Platform, GlobalShaderPermutation))
				{
					if (!GlobalShaderMap->HasShader(GlobalShaderType, PermutationId))
					{
						if (FailureReason)
						{
							FString GlobalShaderTypeName = GlobalShaderType ? GlobalShaderType->GetFName().ToString() : FString("Unknown shader type");
							*FailureReason = FString::Printf(TEXT("Failed to find global shader \"%s\", permutation %d"), *GlobalShaderTypeName, PermutationId);
						}

						return false;
					}
				}
			}
		}

		// Then the pipelines as it may be sharing shaders
		for (TLinkedList<FShaderPipelineType*>::TIterator ShaderPipelineIt(FShaderPipelineType::GetTypeList()); ShaderPipelineIt; ShaderPipelineIt.Next())
		{
			const FShaderPipelineType* Pipeline = *ShaderPipelineIt;
			if (Pipeline->IsGlobalTypePipeline())
			{
				auto& Stages = Pipeline->GetStages();
				int32 NumStagesNeeded = 0;
				for (const FShaderType* Shader : Stages)
				{
					const FGlobalShaderType* GlobalShaderType = Shader->GetGlobalShaderType();
					if (ShouldCacheGlobalShaderTypeName(GlobalShaderType, kUniqueShaderPermutationId, TypeNameSubstring, Platform, GlobalShaderPermutation))
					{
						++NumStagesNeeded;
					}
					else
					{
						break;
					}
				}

				if (NumStagesNeeded == Stages.Num())
				{
					if (!GlobalShaderMap->HasShaderPipeline(Pipeline))
					{
						if (FailureReason)
						{
							*FailureReason = FString::Printf(TEXT("Failed to find global pipeline \"%s\""), *Pipeline->GetFName().ToString());
						}

						return false;
					}
				}
			}
		}
	}

	return true;
}


bool IsGlobalShaderMapComplete(const TCHAR* TypeNameSubstring)
{
	for (int32 i = 0; i < SP_NumPlatforms; ++i)
	{
		EShaderPlatform Platform = (EShaderPlatform)i;

		FGlobalShaderMap* GlobalShaderMap = GGlobalShaderMap[Platform];

		if (!IsGlobalShaderMapComplete(TypeNameSubstring, GlobalShaderMap, Platform))
		{
			return false;
		}
	}

	return true;
}

static bool TryLoadCookedGlobalShaderMap(EShaderPlatform Platform, FScopedSlowTask& SlowTask)
{
	SlowTask.EnterProgressFrame(50);

	bool bLoadedFromCacheFile = false;

	// Load from the override global shaders first, this allows us to hot reload in cooked / pak builds
	TArray<uint8> GlobalShaderData;
	const bool bAllowOverrideGlobalShaders = !WITH_EDITOR && !UE_BUILD_SHIPPING;
	if (bAllowOverrideGlobalShaders)
	{
		FString OverrideGlobalShaderCacheFilename = GetGlobalShaderCacheOverrideFilename(Platform);
		FPaths::MakeStandardFilename(OverrideGlobalShaderCacheFilename);

		bool bFileExist = IFileManager::Get().FileExists(*OverrideGlobalShaderCacheFilename);

		if (!bFileExist)
		{
			UE_LOGF(LogShaders, Display, "%ls doesn't exists", *OverrideGlobalShaderCacheFilename);
		}
		else
		{
			bLoadedFromCacheFile = FFileHelper::LoadFileToArray(GlobalShaderData, *OverrideGlobalShaderCacheFilename, FILEREAD_Silent);

			if (bLoadedFromCacheFile)
			{
				UE_LOGF(LogShaders, Display, "%ls has been loaded successfully", *OverrideGlobalShaderCacheFilename);
			}
			else
			{
				UE_LOGF(LogShaders, Error, "%ls failed to load", *OverrideGlobalShaderCacheFilename);
			}
		}
	}

	// is the data already loaded?
	int64 PreloadedSize = 0;
	void* PreloadedData = nullptr;
	if (!bLoadedFromCacheFile)
	{
		PreloadedData = GGlobalShaderPreLoadFile.TakeOwnershipOfLoadedData(&PreloadedSize);
	}

	if (PreloadedData != nullptr)
	{
		FLargeMemoryReader MemoryReader((uint8*)PreloadedData, PreloadedSize, ELargeMemoryReaderFlags::TakeOwnership);
		GGlobalShaderMap[Platform]->LoadFromGlobalArchive(MemoryReader);
		bLoadedFromCacheFile = true;
	}
	else
	{
		FString GlobalShaderCacheFilename = FPaths::GetRelativePathToRoot() / GetGlobalShaderCacheFilename(Platform);
		FPaths::MakeStandardFilename(GlobalShaderCacheFilename);
		if (!bLoadedFromCacheFile)
		{
			bLoadedFromCacheFile = FFileHelper::LoadFileToArray(GlobalShaderData, *GlobalShaderCacheFilename, FILEREAD_Silent);
		}

		if (bLoadedFromCacheFile)
		{
			FMemoryReader MemoryReader(GlobalShaderData);
			GGlobalShaderMap[Platform]->LoadFromGlobalArchive(MemoryReader);
		}
	}

	return bLoadedFromCacheFile;
}

void CompileGlobalShaderMap(EShaderPlatform Platform, const ITargetPlatform* TargetPlatform, bool bRefreshShaderMap)
{
	LLM_SCOPE_RENDER_RESOURCE(TEXT("GlobalShaderMap"));

	// No global shaders needed on dedicated server or clients that use NullRHI. Note that cook commandlet needs to have them, even if it is not allowed to render otherwise.
	if (FPlatformProperties::IsServerOnly() || (!IsRunningCommandlet() && !FApp::CanEverRender()))
	{
		if (!GGlobalShaderMap[Platform])
		{
			GGlobalShaderMap[Platform] = new FGlobalShaderMap(Platform);
		}
		return;
	}

	if (bRefreshShaderMap || GGlobalShaderTargetPlatform[Platform] != TargetPlatform)
	{
		// defer the deletion the current global shader map, delete the previous one if it is still valid
		delete GGlobalShaderMap_DeferredDeleteCopy[Platform];	// deleting null is Okay
		GGlobalShaderMap_DeferredDeleteCopy[Platform] = GGlobalShaderMap[Platform];
		GGlobalShaderMap[Platform] = nullptr;

		GGlobalShaderTargetPlatform[Platform] = TargetPlatform;

		// make sure we look for updated shader source files
		FlushShaderFileCache();
	}

#if WITH_ODSC
	// First try to load the global shader map with ODSC if it's connected. TryLoadGlobalShaders will set GGlobalShaderMap[Platform]
	if (!GGlobalShaderMap[Platform] && FODSCManager::IsODSCActive())
	{
		UE_LOGF(LogShaders, Display, "Trying to load global shaders from ODSC ...");
		GODSCManager->TryLoadGlobalShaders(Platform);
		UE_LOGF(LogShaders, Display, "Global shaders from ODSC: %ls", (GGlobalShaderMap[Platform] != nullptr) ? TEXT("success") : TEXT("failed"));
	}
#endif

	// If the global shader map hasn't been created yet, create it.
	if (!GGlobalShaderMap[Platform])
	{
		DECLARE_SCOPE_CYCLE_COUNTER(TEXT("GetGlobalShaderMap"), STAT_GetGlobalShaderMap, STATGROUP_LoadTime);
		// GetGlobalShaderMap is called the first time during startup in the main thread.
		check(IsInGameThread());

		FScopedSlowTask SlowTask(70, LOCTEXT("CreateGlobalShaderMap", "Creating Global Shader Map..."));

		// verify that all shader source files are intact
		SlowTask.EnterProgressFrame(20, LOCTEXT("VerifyShaderSourceFiles", "Verifying Global Shader source files..."));
		VerifyShaderSourceFiles(Platform);

		GGlobalShaderMap[Platform] = new FGlobalShaderMap(Platform);

		bool bShaderMapIsBeingCompiled = false;

		// Try to load the global shaders from a local cache file if it exists
		// We always try this first, even when running in the editor or if shader compiler is enabled
		// It's always possible we'll find a cooked local cache
		const bool bLoadedFromCacheFile = TryLoadCookedGlobalShaderMap(Platform, SlowTask);
#if WITH_EDITOR
		const bool bAllowShaderCompiling = !FPlatformProperties::RequiresCookedData() && AllowShaderCompiling();
#else
		const bool bAllowShaderCompiling = false;
#endif

#if WITH_EDITOR
		if (!bLoadedFromCacheFile && bAllowShaderCompiling)
		{
			PrimeGlobalShaderSourceHashCache(Platform, TargetPlatform);
			
			// Ensure we've generated AutogenShaderHeaders.ush
			FShaderCompileUtilities::GenerateBrdfHeaders(Platform);

			// If we didn't find cooked shaders, we can try loading from the DDC or compiling them if supported by the current configuration
			FGlobalShaderMapId ShaderMapId(Platform, TargetPlatform);

			const int32 ShaderFilenameNum = ShaderMapId.GetShaderFilenameToDependeciesMap().Num();
			const float ProgressStep = 25.0f / ShaderFilenameNum;

			// If NoShaderDDC then don't check for a material the first time we encounter it to simulate
			// a cold DDC
			static bool bNoShaderDDC =
				FParse::Param(FCommandLine::Get(), TEXT("noshaderddc")) || 
				FParse::Param(FCommandLine::Get(), TEXT("noglobalshaderddc"));

			const bool bTempNoShaderDDC = bNoShaderDDC;

			{
				using namespace UE::DerivedData;

				int32 BufferIndex = 0;
				TArray<FCacheGetRequest> Requests;

				// Submit DDC requests.
				SlowTask.EnterProgressFrame(ProgressStep, LOCTEXT("SubmitDDCRequests", "Submitting global shader DDC Requests..."));
				for (const auto& ShaderFilenameDependencies : ShaderMapId.GetShaderFilenameToDependeciesMap())
				{
					FCacheGetRequest& Request = Requests.AddDefaulted_GetRef();
					Request.Name = GetGlobalShaderMapName(ShaderMapId, Platform, ShaderFilenameDependencies.Key);
					Request.Key = GetGlobalShaderMapKey(ShaderMapId, Platform, TargetPlatform, ShaderFilenameDependencies.Value);
					Request.UserData = uint64(BufferIndex);
					++BufferIndex;

					if (UNLIKELY(ShouldDumpShaderDDCKeys()))
					{
						const FString DataKey = GetGlobalShaderMapKeyString(ShaderMapId, Platform, ShaderFilenameDependencies.Value);
						// For global shaders, we dump the key multiple times (once for each shader type) so they will live on disk alongside
						// other shader debug artifacts.
						for (const FShaderTypeDependency& ShaderTypeDependency : ShaderFilenameDependencies.Value)
						{
							const FShaderType* ShaderType = FindShaderTypeByName(ShaderTypeDependency.ShaderTypeName);
							TStringBuilder<128> GroupNameBuilder;
							GroupNameBuilder << TEXT("Global");
							FPathViews::Append(GroupNameBuilder, ShaderType->GetName());
							DumpShaderDDCKeyToFile(Platform, ShaderMapId.WithEditorOnly(), GroupNameBuilder.ToString(), DataKey);
						}
					}
				}

				int32 DDCHits = 0;
				int32 DDCMisses = 0;

				// Process finished DDC requests.
				SlowTask.EnterProgressFrame(ProgressStep, LOCTEXT("ProcessDDCRequests", "Processing global shader DDC requests..."));
				TArray<FShaderCacheLoadContext> GlobalShaderMapLoads;
				GlobalShaderMapLoads.SetNum(Requests.Num());
				{
					COOK_STAT(auto Timer = GlobalShaderCookStats::UsageStats.TimeSyncWork());
					COOK_STAT(Timer.TrackCyclesOnly());
					FRequestOwner BlockingOwner(EPriority::Blocking);
					GetCache().Get(Requests, BlockingOwner, [&GlobalShaderMapLoads, &bTempNoShaderDDC](FCacheGetResponse&& Response)
					{
						if (bTempNoShaderDDC)
						{
							return;
						}
						if (Response.Status == EStatus::Ok)
						{
							GlobalShaderMapLoads[int32(Response.UserData)].ReadFromRecord(Response.Record);
						}
					});
					{
						TRACE_CPUPROFILER_EVENT_SCOPE(WaitForDDC);
						BlockingOwner.Wait();
					}
				}

				BufferIndex = 0;
				for (const auto& ShaderFilenameDependencies : ShaderMapId.GetShaderFilenameToDependeciesMap())
				{
					COOK_STAT(auto Timer = GlobalShaderCookStats::UsageStats.TimeSyncWork());
					if (GlobalShaderMapLoads[BufferIndex].ShaderObjectData)
					{
						GGlobalShaderMap[Platform]->AddSection(FGlobalShaderMapSection::CreateFromCache(GlobalShaderMapLoads[BufferIndex]));
						COOK_STAT(Timer.AddHit(int64(GlobalShaderMapLoads[BufferIndex].GetSerializedSize())));
						DDCHits++;
					}
					else
					{
						// it's a miss, but we haven't built anything yet. Save the counting until we actually have it built.
						COOK_STAT(Timer.TrackCyclesOnly());
						bShaderMapIsBeingCompiled = true;
						DDCMisses++;
					}
					++BufferIndex;
				}

				GShaderCompilerStats->AddDDCHit(DDCHits);
				GShaderCompilerStats->AddDDCMiss(DDCMisses);
			}
		}
#endif // WITH_EDITOR
		
		if (!bLoadedFromCacheFile && !bAllowShaderCompiling)
		{
			// Failed to load cooked shaders, and no support for compiling
			// Handle this gracefully and exit.
			const FString GlobalShaderCacheFilename = FPaths::GetRelativePathToRoot() / GetGlobalShaderCacheFilename(Platform);
			const FString SandboxPath = IFileManager::Get().ConvertToAbsolutePathForExternalAppForWrite(*GlobalShaderCacheFilename);
			// This can be too early to localize in some situations.
			const FText Message = FText::Format(NSLOCTEXT("Engine", "GlobalShaderCacheFileMissing", "The global shader cache file '{0}' is missing.\n\nYour application is built to load COOKED content. No COOKED content was found; This usually means you did not cook content for this build.\nIt also may indicate missing cooked data for a shader platform(e.g., OpenGL under Windows): Make sure your platform's packaging settings include this Targeted RHI.\n\nAlternatively build and run the UNCOOKED version instead."), FText::FromString(SandboxPath));
			if (FPlatformProperties::SupportsWindowedMode())
			{
				UE_LOGF(LogShaders, Error, "%ls", *Message.ToString());
				FMessageDialog::Open(EAppMsgType::Ok, Message);
				FPlatformMisc::RequestExit(false, TEXT("CompileGlobalShaderMap"));
				return;
			}
			else
			{
				UE_LOGF(LogShaders, Fatal, "%ls", *Message.ToString());
			}
		}

		// If any shaders weren't loaded, compile them now.
		VerifyGlobalShaders(Platform, TargetPlatform, bLoadedFromCacheFile);

		if (CreateShadersOnLoad() && Platform == GMaxRHIShaderPlatform)
		{
			GGlobalShaderMap[Platform]->BeginCreateAllShaders();
		}

		// While we're early in the game's startup, create certain global shaders that may be later created on random threads otherwise. 
		if (!bShaderMapIsBeingCompiled && !GRHISupportsMultithreadedShaderCreation)
		{
			ENQUEUE_RENDER_COMMAND(CreateRecursiveShaders)([](FRHICommandListImmediate&)
			{
				CreateRecursiveShaders();
			});
		}
	}
}

void CompileGlobalShaderMap(EShaderPlatform Platform, bool bRefreshShaderMap)
{
	CompileGlobalShaderMap(Platform, nullptr, bRefreshShaderMap);
}

void CompileGlobalShaderMap(ERHIFeatureLevel::Type InFeatureLevel, bool bRefreshShaderMap)
{
	EShaderPlatform Platform = GShaderPlatformForFeatureLevel[InFeatureLevel];
	CompileGlobalShaderMap(Platform, nullptr, bRefreshShaderMap);
}

void CompileGlobalShaderMap(bool bRefreshShaderMap)
{
	CompileGlobalShaderMap(GMaxRHIFeatureLevel, bRefreshShaderMap);
}

void ShutdownGlobalShaderMap()
{
	// handle edge case where we get a shutdown before fully initialized (the globals used below are not in a valid state)
	if (!GIsRHIInitialized)
	{
		return;
	}

	// at the point this function is called (during the shutdown process) we do not expect any outstanding work that could potentially be still referencing
	// global shaders, so we are not deferring the deletion (via GGlobalShaderMap_DeferredDeleteCopy) like we do during the shader recompilation.
	EShaderPlatform Platform = GShaderPlatformForFeatureLevel[GMaxRHIFeatureLevel];
	if (GGlobalShaderMap[Platform] != nullptr)
	{
		GGlobalShaderMap[Platform]->ReleaseAllSections();

		delete GGlobalShaderMap[Platform];
		GGlobalShaderMap[Platform] = nullptr;
	}
}

void ReloadGlobalShaders()
{
	UE_LOGF(LogShaders, Display, "Reloading global shaders...");

	// Flush pending accesses to the existing global shaders.
	FlushRenderingCommands();

	UMaterialInterface::IterateOverActiveFeatureLevels(
		[&](ERHIFeatureLevel::Type InFeatureLevel)
		{
			auto ShaderPlatform = GShaderPlatformForFeatureLevel[InFeatureLevel];
			GetGlobalShaderMap(ShaderPlatform)->ReleaseAllSections();
			CompileGlobalShaderMap(InFeatureLevel, true);
			VerifyGlobalShaders(ShaderPlatform, nullptr, false);
		}
	);

	// Invalidate global bound shader states so they will be created with the new shaders the next time they are set (in SetGlobalBoundShaderState)
	for (TLinkedList<FGlobalBoundShaderStateResource*>::TIterator It(FGlobalBoundShaderStateResource::GetGlobalBoundShaderStateList()); It; It.Next())
	{
		BeginUpdateResourceRHI(*It);
	}

	PropagateGlobalShadersToAllPrimitives();
}

static FAutoConsoleCommand CCmdReloadGlobalShaders = FAutoConsoleCommand(
	TEXT("ReloadGlobalShaders"),
	TEXT("Reloads the global shaders file"),
	FConsoleCommandDelegate::CreateStatic(ReloadGlobalShaders)
);

void SetGlobalShaderCacheOverrideDirectory(const TArray<FString>& Args)
{
	if (Args.Num() == 0)
	{
		UE_LOGF(LogShaders, Error, "Failed to set GGlobalShaderCacheOverrideDirectory without any arguments");
		return; 
	}
	
	GGlobalShaderCacheOverrideDirectory = Args[0];
	UE_LOGF(LogShaders, Log, "GGlobalShaderCacheOverrideDirectory = %ls", *GGlobalShaderCacheOverrideDirectory);
}

static FAutoConsoleCommand CCmdSetGlobalShaderCacheOverrideDirectory = FAutoConsoleCommand(
	TEXT("SetGlobalShaderCacheOverrideDirectory"),
	TEXT("Set the directory to read the override global shader map file from."),
	FConsoleCommandWithArgsDelegate::CreateStatic(SetGlobalShaderCacheOverrideDirectory));

bool RecompileChangedShadersForPlatform(const FString& PlatformName)
{
	// figure out what shader platforms to recompile
	ITargetPlatformManagerModule* TPM = GetTargetPlatformManager();
	ITargetPlatform* TargetPlatform = TPM->FindTargetPlatform(PlatformName);
	if (TargetPlatform == nullptr)
	{
		UE_LOGF(LogShaders, Display, "Failed to find target platform module for %ls", *PlatformName);
		return false;
	}

	TArray<FName> DesiredShaderFormats;
	TargetPlatform->GetAllTargetedShaderFormats(DesiredShaderFormats);

	// figure out which shaders are out of date
	TArray<const FShaderType*> OutdatedShaderTypes;
	TArray<const FVertexFactoryType*> OutdatedFactoryTypes;
	TArray<const FShaderPipelineType*> OutdatedShaderPipelineTypes;

	// Pick up new changes to shader files
	FlushShaderFileCache();

	GetOutdatedShaderTypes(OutdatedShaderTypes, OutdatedShaderPipelineTypes, OutdatedFactoryTypes);
	UE_LOGF(LogShaders, Display, "We found %d out of date shader types, %d outdated pipeline types, and %d out of date VF types!", OutdatedShaderTypes.Num(), OutdatedShaderPipelineTypes.Num(), OutdatedFactoryTypes.Num());

#if WITH_EDITOR
	UpdateReferencedUniformBufferNames(OutdatedShaderTypes, OutdatedFactoryTypes, OutdatedShaderPipelineTypes);
#endif

	for (int32 FormatIndex = 0; FormatIndex < DesiredShaderFormats.Num(); FormatIndex++)
	{
		// get the shader platform enum
		const EShaderPlatform ShaderPlatform = ShaderFormatToLegacyShaderPlatform(DesiredShaderFormats[FormatIndex]);

		// Only compile for the desired platform if requested
		// Kick off global shader recompiles
		BeginRecompileGlobalShaders(OutdatedShaderTypes, OutdatedShaderPipelineTypes, ShaderPlatform);

		// Block on global shaders
		FinishRecompileGlobalShaders();
#if WITH_EDITOR
		// we only want to actually compile mesh shaders if we have out of date ones
		if (OutdatedShaderTypes.Num() || OutdatedFactoryTypes.Num())
		{
			for (TObjectIterator<UMaterialInterface> It; It; ++It)
			{
				(*It)->ClearCachedCookedPlatformData(TargetPlatform);
			}
		}
#endif // WITH_EDITOR
	}

	if (OutdatedFactoryTypes.Num() || OutdatedShaderTypes.Num())
	{
		return true;
	}
	return false;
}

extern ENGINE_API const TCHAR* ODSCCmdEnumToString(ODSCRecompileCommand Cmd)
{
	switch (Cmd)
	{
	case ODSCRecompileCommand::None:
		return TEXT("None");
	case ODSCRecompileCommand::Changed:
		return TEXT("Change");
	case ODSCRecompileCommand::Global:
		return TEXT("Global");
	case ODSCRecompileCommand::Material:
		return TEXT("Material");
	case ODSCRecompileCommand::SingleShader:
		return TEXT("SingleShader");
	case ODSCRecompileCommand::ResetMaterialCache:
			return TEXT("ResetMaterialCache");
	}
	ensure(false);
	return TEXT("Unknown");
}

void BeginRecompileGlobalShaders(const TArray<const FShaderType*>& OutdatedShaderTypes, const TArray<const FShaderPipelineType*>& OutdatedShaderPipelineTypes, EShaderPlatform ShaderPlatform, const ITargetPlatform* TargetPlatform, const FShaderCompilerFlags& InExtraCompilerFlags)
{
#if WITH_EDITOR
	if (!FPlatformProperties::RequiresCookedData())
	{
		// Flush pending accesses to the existing global shaders.
		FlushRenderingCommands();

		// Calling CompileGlobalShaderMap will force starting the compile jobs if the map is empty (by calling VerifyGlobalShaders)
		CompileGlobalShaderMap(ShaderPlatform, TargetPlatform, false);
		FGlobalShaderMap* GlobalShaderMap = GetGlobalShaderMap(ShaderPlatform);

		// Now check if there is any work to be done wrt outdates types
		if (OutdatedShaderTypes.Num() > 0 || OutdatedShaderPipelineTypes.Num() > 0)
		{
			VerifyGlobalShaders(ShaderPlatform, TargetPlatform, false, &OutdatedShaderTypes, &OutdatedShaderPipelineTypes, InExtraCompilerFlags);
		}
	}
#endif
}

void FinishRecompileGlobalShaders()
{
	// Block until global shaders have been compiled and processed
	GShaderCompilingManager->ProcessAsyncResults(false, true);
}

void LoadGlobalShadersForRemoteRecompile(FArchive& Ar, EShaderPlatform ShaderPlatform)
{
	uint8 bIsValid = 0;
	Ar << bIsValid;

	if (bIsValid)
	{
		FlushRenderingCommands();

		FGlobalShaderMap* NewGlobalShaderMap = new FGlobalShaderMap(ShaderPlatform);
		if (NewGlobalShaderMap)
		{
			NewGlobalShaderMap->LoadFromGlobalArchive(Ar);

			FString FailureReason;
			bool bIsNewGlobalShaderMapComplete = IsGlobalShaderMapComplete(nullptr, NewGlobalShaderMap, ShaderPlatform, &FailureReason);

			if (bIsNewGlobalShaderMapComplete)
			{
				if (GGlobalShaderMap[ShaderPlatform])
				{
					GGlobalShaderMap[ShaderPlatform]->ReleaseAllSections();
					delete GGlobalShaderMap[ShaderPlatform];
					GGlobalShaderMap[ShaderPlatform] = nullptr;
				}
				GGlobalShaderMap[ShaderPlatform] = NewGlobalShaderMap;

				VerifyGlobalShaders(ShaderPlatform, nullptr, false);

				// Invalidate global bound shader states so they will be created with the new shaders the next time they are set (in SetGlobalBoundShaderState)
				for (TLinkedList<FGlobalBoundShaderStateResource*>::TIterator It(FGlobalBoundShaderStateResource::GetGlobalBoundShaderStateList()); It; It.Next())
				{
					BeginUpdateResourceRHI(*It);
				}

				PropagateGlobalShadersToAllPrimitives();
			}
			else
			{
				FString ErrorMessage = FString::Printf(TEXT("New global shader map is incomplete and will not be used. Reason:\n%s\n"
													        "Please check the ODSC server log & that client/editor are compiled"), *FailureReason);

				UE_LOGF(LogShaderCompilers, Error, "%ls", *ErrorMessage);
#if WITH_ODSC
				FODSCManager::ReportODSCError(ErrorMessage);
#endif
				
				delete NewGlobalShaderMap;
			}
		}
	}
}

#undef LOCTEXT_NAMESPACE
