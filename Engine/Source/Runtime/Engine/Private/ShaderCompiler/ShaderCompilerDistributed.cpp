// Copyright Epic Games, Inc. All Rights Reserved.

#include "DistributedBuildControllerInterface.h"
#include "Async/Future.h"
#include "HAL/FileManager.h"
#include "Misc/CommandLine.h"
#include "Misc/FileHelper.h"
#include "Misc/StringBuilder.h"
#include "Serialization/MemoryReader.h"
#include "Serialization/MemoryWriter.h"
#include "ShaderCompiler.h"
#include "ShaderCompilerJobTypes.h"
#include "ShaderCompiler/ShaderCompilerInternal.h"
#include "ShaderCompileWorkerUtil.h"

namespace DistributedShaderCompilerVariables
{
	//TODO: Remove the XGE doublet
	int32 MinBatchSize = 50;
	FAutoConsoleVariableRef CVarXGEShaderCompileMinBatchSize(
        TEXT("r.XGEShaderCompile.MinBatchSize"),
        MinBatchSize,
        TEXT("This CVar is deprecated, please use r.ShaderCompiler.DistributedMinBatchSize"),
        ECVF_Default);

	FAutoConsoleVariableRef CVarDistributedMinBatchSize(
		TEXT("r.ShaderCompiler.DistributedMinBatchSize"),
		MinBatchSize,
		TEXT("Minimum number of shaders to compile with a distributed controller.\n")
		TEXT("Smaller number of shaders will compile locally."),
		ECVF_Default);

	static int32 GDistributedJobDescriptionLevel = 1;
	static FAutoConsoleVariableRef CVarDistributedJobDescriptionLevel(
		TEXT("r.ShaderCompiler.DistributedJobDescriptionLevel"),
		GDistributedJobDescriptionLevel,
		TEXT("Sets the level of descriptive details for each distributed job batch. The following modes are supported:\n")
		TEXT(" Mode 0: Disabled.\n")
		TEXT(" Mode 1: Basic information of the first 20 compile jobs per batch (Default).\n")
		TEXT(" Mode 2: Additional information of the shader format per compile job.\n")
		TEXT("This will show up in the UBA trace files. By default 0.")
	);

}

extern bool GShaderCompilerDumpWorkerDiagnostics;

bool FShaderCompileDistributedThreadRunnable_Interface::IsSupported()
{
	//TODO Handle Generic response
	return true;
}

class FDistributedShaderCompilerTask
{
public:
	TFuture<FDistributedBuildTaskResult> Future;
	TArray<FShaderCommonCompileJobPtr> ShaderJobs;
	FString InputFilePath;
	FString OutputFilePath;

	FDistributedShaderCompilerTask(TFuture<FDistributedBuildTaskResult>&& Future,TArray<FShaderCommonCompileJobPtr>&& ShaderJobs, FString&& InputFilePath, FString&& OutputFilePath)
		: Future(MoveTemp(Future))
		, ShaderJobs(MoveTemp(ShaderJobs))
		, InputFilePath(MoveTemp(InputFilePath))
		, OutputFilePath(MoveTemp(OutputFilePath))
	{}
};

/** Initialization constructor. */
FShaderCompileDistributedThreadRunnable_Interface::FShaderCompileDistributedThreadRunnable_Interface(class FShaderCompilingManager* InManager, IDistributedBuildController& InController)
	: FShaderCompileThreadRunnableBase(InManager)
	, NumDispatchedJobs(0)
	, CachedController(InController)
{
}

FShaderCompileDistributedThreadRunnable_Interface::~FShaderCompileDistributedThreadRunnable_Interface()
{
}

static FString BuildCompactTaskDescriptionForJob(const FShaderCommonCompileJob& Job)
{
	FString JobDescription;
	if (const FShaderCompileJob* SingleJob = Job.GetSingleShaderJob())
	{
		JobDescription = SingleJob->Input.DebugGroupName;
		if (DistributedShaderCompilerVariables::GDistributedJobDescriptionLevel >= 2)
		{
			JobDescription += FString::Printf(TEXT("(%s)"), *SingleJob->Input.ShaderFormat.ToString());
		}
	}
	else if (const FShaderPipelineCompileJob* PipelineJob = Job.GetShaderPipelineJob())
	{
		JobDescription = TEXT("Stages:");
		for (int32 StageJobIndex = 0; StageJobIndex < PipelineJob->StageJobs.Num(); ++StageJobIndex)
		{
			if (StageJobIndex > 0)
			{
				JobDescription += TEXT(",");
			}
			JobDescription += BuildCompactTaskDescriptionForJob(*PipelineJob->StageJobs[StageJobIndex]);
		}
	}
	return JobDescription;
}

// Builds a compact description of the shader compile task that will show up in UBA trace files for instance.
// It shall contain a brief summary of the shaders being compiled to diagnose issues with overly long remote jobs.
static FString BuildCompactTaskDescription(const TArray<FShaderCommonCompileJobPtr>& JobsToSerialize)
{
	FString Description;

	if (!JobsToSerialize.IsEmpty())
	{
		constexpr int32 MaxNumJobsInDescription = 20;
		const int32 NumJobsInDescription = JobsToSerialize.Num() > MaxNumJobsInDescription ? MaxNumJobsInDescription - 1 : JobsToSerialize.Num();
		for (int32 JobIndex = 0; JobIndex < NumJobsInDescription; ++JobIndex)
		{
			if (JobIndex > 0)
			{
				Description += TEXT("\n");
			}
			Description += BuildCompactTaskDescriptionForJob(*JobsToSerialize[JobIndex]);
		}
		if (JobsToSerialize.Num() > NumJobsInDescription)
		{
			Description += FString::Printf(TEXT("\n%d more shaders ...\n"), JobsToSerialize.Num() - NumJobsInDescription);
		}
	}

	return Description;
}

static bool IsCharValidForFilename(TCHAR Ch)
{
	const TCHAR* ValidSpecialChars = TEXT("_-+()[]");
	return TChar<TCHAR>::IsAlnum(Ch) || FCString::Strchr(ValidSpecialChars, Ch) != nullptr;
}

static void ConvertDebugNameToFilename(TStringBuilder<1024>& StringBuilder, const FString& Name)
{
	for (TCHAR Ch : Name)
	{
		StringBuilder << (IsCharValidForFilename(Ch) ? Ch : TEXT('-'));
	}
}

static void BuildDescriptiveTaskFilename(TStringBuilder<1024>& StringBuilder, const TArray<FShaderCommonCompileJobPtr>& JobsToSerialize, int32 BasePathLen, int32 PathLimit)
{
	StringBuilder << TEXT(".");

	if (JobsToSerialize.Num() == 1)
	{
		// Decorate filename with single job description
		if (const FShaderCompileJob* FirstSingleJob = JobsToSerialize[0]->GetSingleShaderJob())
		{
			const int32 BaseFilenameLen = BasePathLen + StringBuilder.Len();
			const int32 TotalFilenameLen = BaseFilenameLen + FirstSingleJob->Input.DebugGroupName.Len();
			if (TotalFilenameLen > PathLimit)
			{
				const int32 RemainingLen = PathLimit - BaseFilenameLen;

				// If even a shortened task filename with just 5 characters were to exceed the path limit,
				// don't bother trying to construct a descriptive task name, just fall back to default.
				constexpr int32 kMinFilenameSuffixLen = 5;
				if (RemainingLen >= kMinFilenameSuffixLen)
				{
					const int32 RemainingLenHalfPart = (RemainingLen - 3) / 2;
					FString ShortenedDebugGroupName = FString::Printf(TEXT("%s---%s"), *FirstSingleJob->Input.DebugGroupName.Left(RemainingLenHalfPart), *FirstSingleJob->Input.DebugGroupName.Right(RemainingLenHalfPart));
					ConvertDebugNameToFilename(StringBuilder, ShortenedDebugGroupName);
					return;
				}
			}
			else
			{
				ConvertDebugNameToFilename(StringBuilder, FirstSingleJob->Input.DebugGroupName);
				return;
			}
		}
	}

	// Decorate filename with number of jobs
	StringBuilder << TEXT("j-") << JobsToSerialize.Num();
}

// for reasons, FMemoryWriter/FMemoryReader serialize FNames differently than the file equivalents do. we need to revert this behaviour
// back to the base implementation so when writing/reading back the virtualized files it is in the same format as is typical for files.
class FShaderTaskFileMemoryWriter : public FMemoryWriter
{
	using FMemoryWriter::FMemoryWriter;
	virtual FArchive& operator<<( class FName& N ) override { return FArchive::operator<<(N); }
};

class FShaderTaskFileMemoryReader : public FMemoryReader
{
	using FMemoryReader::FMemoryReader;
	virtual FArchive& operator<<(class FName& N) override { return FArchive::operator<<(N); }
};

void FShaderCompileDistributedThreadRunnable_Interface::DispatchShaderCompileJobsBatch(TArray<FShaderCommonCompileJobPtr>& JobsToSerialize)
{
	// Generate unique filename for shader compiler I/O files
	FString BaseFilePath = CachedController.CreateUniqueFilePath();

	constexpr int32 kMaxFilePathSuffixLenWithNullChar = 5; // for ".out\0"
	const int32 PathLimit = FPlatformMisc::GetExternalAppMaxPathLength() - kMaxFilePathSuffixLenWithNullChar;
	if (BaseFilePath.Len() > PathLimit)
	{
		UE_LOGF(LogShaderCompilers, Error, "Distributed job batch exceeded path limit: Length=%d, Limit=%d, Path=%ls", BaseFilePath.Len(), PathLimit, *BaseFilePath);
	}

	TStringBuilder<1024> BaseFilePathBuilder;
	BuildDescriptiveTaskFilename(BaseFilePathBuilder, JobsToSerialize, BaseFilePath.Len(), PathLimit);
	BaseFilePath += BaseFilePathBuilder.ToView();

	// Ensure the constructed input and output filenames don't exceed platform path limit
	if (BaseFilePath.Len() > PathLimit)
	{
		BaseFilePath.LeftChopInline(BaseFilePath.Len() - PathLimit);
	}

	FString InputFilePath = BaseFilePath + TEXT(".in");
	FString OutputFilePath = BaseFilePath + TEXT(".out");

	// Set up remote task
	const FString WorkingDirectory = FPaths::GetPath(InputFilePath);

	// Serialize the jobs to the input file
	GShaderCompilerStats->RegisterJobBatch(JobsToSerialize.Num(), FShaderCompilerStats::EExecutionType::Distributed);

	FTaskCommandData TaskCommandData;
	TaskCommandData.Command = Manager->ShaderCompileWorkerName;
	TaskCommandData.WorkingDirectory = WorkingDirectory;
	TaskCommandData.DispatcherPID = Manager->ProcessId;
	TaskCommandData.InputFileName = InputFilePath;
	TaskCommandData.OutputFileName = OutputFilePath;
	TStringBuilder<64> SubprocessCommandLine;
	FCommandLine::BuildSubprocessCommandLine(ECommandLineArgumentFlags::ProgramContext, false /*bOnlyInherited*/, SubprocessCommandLine);

	// Build the full ShaderCompileWorker command line here.
	TaskCommandData.ExtraCommandArgs = FString::Printf(
		TEXT("\"%s/\" %d 0 \"%s\" \"%s\" %s%s"),
		*WorkingDirectory,
		Manager->ProcessId,
		*FPaths::GetCleanFilename(InputFilePath),
		*FPaths::GetCleanFilename(OutputFilePath),
		*SubprocessCommandLine,
		GIsBuildMachine ? TEXT(" -buildmachine") : TEXT(""));

	{
		TUniquePtr<FArchive> InputFileAr;
		FShaderCompileWorkerUtil::EWriteTasksFlags Flags = FShaderCompileWorkerUtil::EWriteTasksFlags::None;
		if (CachedController.SupportsVirtualFiles())
		{
			InputFileAr = TUniquePtr<FArchive>(new FShaderTaskFileMemoryWriter(TaskCommandData.InputFileData));
			// when using virtual files, we create additional virtual files for all shader sources pointing to the copy owned by the job, 
			// rather than duplicating them in memory
			Flags |= FShaderCompileWorkerUtil::EWriteTasksFlags::SkipSource;
			// we also compress the task file to further reduce the memory impact
			Flags |= FShaderCompileWorkerUtil::EWriteTasksFlags::CompressTaskFile;
		}
		else
		{
			InputFileAr = TUniquePtr<FArchive>(IFileManager::Get().CreateFileWriter(*InputFilePath, FILEWRITE_EvenIfReadOnly | FILEWRITE_NoFail));
		}
		FShaderCompileWorkerUtil::WriteTasks(JobsToSerialize, *InputFileAr, Flags, CachedController.SupportsVirtualFiles() ? &TaskCommandData.SourceData : nullptr);
	}

	if (!TaskCommandData.SourceData.IsEmpty())
	{
		// SourceData contains non-owning FMemoryViews into compressed source buffers owned by the compile jobs.
		// Keep the jobs alive via a refcounted job pointer so the views remain valid for the lifetime of the task.
		// (handle edge case where distributed build impl is spawning tasks while the owning jobs are being cancelled)
		TaskCommandData.SourceDataOwner = MakeShared<TArray<FShaderCommonCompileJobPtr>, ESPMode::ThreadSafe>(JobsToSerialize);
	}

	// Kick off the job
	NumDispatchedJobs += JobsToSerialize.Num();


	// Register any debug info paths that may be written to as additional output folders
	// Without this remote tasks can incorrectly report that the debug info paths do not exist
	for (const FShaderCommonCompileJobPtr& Job : JobsToSerialize)
	{
		Job->ForEachSingleShaderJob([&TaskCommandData](FShaderCompileJob& SingleJob)
			{
				if (SingleJob.Input.DumpDebugInfoEnabled())
				{
					TaskCommandData.AdditionalOutputFolders.Add(SingleJob.Input.DumpDebugInfoPath);
				};
			});
	}

	if (DistributedShaderCompilerVariables::GDistributedJobDescriptionLevel > 0)
	{
		TaskCommandData.Description = BuildCompactTaskDescription(JobsToSerialize);
	}
	
	DispatchedTasks.Add(
		MakeUnique<FDistributedShaderCompilerTask>(
			CachedController.EnqueueTask(TaskCommandData),
			MoveTemp(JobsToSerialize),
			MoveTemp(InputFilePath),
			MoveTemp(OutputFilePath)
		)
	);

	FDistributedBuildStats Stats;
	if (CachedController.PollStats(Stats))
	{
		GShaderCompilerStats->RegisterDistributedBuildStats(Stats);
	}
}

static int32 GetNumberOfShaderJobs(const TSparseArray<TUniquePtr<FDistributedShaderCompilerTask>>& InDispatchedTasks)
{
	int32 NumJobs = 0;
	for (auto Iter = InDispatchedTasks.CreateConstIterator(); Iter; ++Iter)
	{
		FDistributedShaderCompilerTask* Task = Iter->Get();
		NumJobs += Task->ShaderJobs.Num();
	}
	return NumJobs;
}

static void ReportShaderCompileWorkerDistributedDiagnostics(const TSparseArray<TUniquePtr<FDistributedShaderCompilerTask>>& InDispatchedTasks, int32 MaxJobsToReport = 50)
{
	UE_LOGF(LogShaderCompilers, Display, "======= ShaderCompileWorker-Distributed Diagnostics =======");

	FString JobDiagnostics;

	const int32 TotalNumJobs = GetNumberOfShaderJobs(InDispatchedTasks);

	int32 TaskIndex = 0, TotalJobsReported = 0, TotalTasksReported = 0;
	for (auto Iter = InDispatchedTasks.CreateConstIterator(); Iter && TotalJobsReported < MaxJobsToReport; ++Iter, ++TotalTasksReported)
	{
		FDistributedShaderCompilerTask* Task = Iter->Get();
		checkf(Task != nullptr, TEXT("Task entries in the sparse array of dispatched distributed shader jobs (FDistributedShaderCompilerTask) must not be null"));
		if (!Task->ShaderJobs.IsEmpty())
		{
			JobDiagnostics.Empty();

			for (int32 JobIndex = 0; JobIndex < Task->ShaderJobs.Num() && TotalJobsReported < MaxJobsToReport; ++JobIndex, ++TotalJobsReported)
			{
				Task->ShaderJobs[JobIndex]->AppendDiagnostics(JobDiagnostics, JobIndex, Task->ShaderJobs.Num(), TEXT("  "));
			}

			UE_LOGF(
				LogShaderCompilers, Display, "Task [%d/%d]:\n%ls",
				TaskIndex + 1, InDispatchedTasks.Num(), *JobDiagnostics
			);
		}
		++TaskIndex;
	}

	if (TotalNumJobs > TotalJobsReported)
	{
		const int32 NumRemainingTasks = InDispatchedTasks.Num() - TotalTasksReported;
		const int32 NumRemainingJobs = TotalNumJobs - TotalJobsReported;
		UE_LOGF(LogShaderCompilers, Display, "%d more %ls, %d more %ls ...", NumRemainingTasks, NumRemainingTasks == 1 ? TEXT("task") : TEXT("tasks"), NumRemainingJobs, NumRemainingJobs == 1 ? TEXT("job") : TEXT("jobs"));
	}
}

static void TransferJobsToLocalWorker(TSparseArray<TUniquePtr<FDistributedShaderCompilerTask>>&& DispatchedTasks)
{
	TArray<FShaderCommonCompileJobPtr> ShaderJobs;

	for (auto Iter = DispatchedTasks.CreateIterator(); Iter; ++Iter)
	{
		FDistributedShaderCompilerTask* Task = Iter->Get();
		checkf(Task != nullptr, TEXT("Task entries in the sparse array of dispatched distributed shader jobs (FDistributedShaderCompilerTask) must not be null"));

		ShaderJobs.Append(Task->ShaderJobs);

		// Delete task input and output files if they exist. Even if the output file appears delayed and we miss to delete it here,
		// they will be deleted alongside the entire working directory of distributed tasks at launch on the next run.
		if (IFileManager::Get().FileExists(*Task->InputFilePath))
		{
			IFileManager::Get().Delete(*Task->InputFilePath, false, true, true);
		}
		if (IFileManager::Get().FileExists(*Task->OutputFilePath))
		{
			IFileManager::Get().Delete(*Task->OutputFilePath, false, true, true);
		}

		Iter.RemoveCurrent();
	}

	GShaderCompilingManager->TransferJobsToLocalWorker(ShaderJobs);
}

int32 FShaderCompileDistributedThreadRunnable_Interface::CompilingLoop()
{
	TArray<FShaderCommonCompileJobPtr> PendingJobs;
	{
		FScopeLock Lock(&Manager->CompileQueueSection);
		const TInterval<int32> PriorityRange = GetPriorityRange();
		for (int32 PriorityIndex = PriorityRange.Max; PriorityIndex >= PriorityRange.Min; --PriorityIndex)
		{
			// Grab as many jobs from the job queue as we can, unless there is no local shader compiling thread to pick up smaller batches
			const EShaderCompileJobPriority Priority = (EShaderCompileJobPriority)PriorityIndex;
			const int32 MinBatchSize = (Priority == EShaderCompileJobPriority::Low || Manager->IsExclusiveDistributedCompilingEnabled()) ? 1 : DistributedShaderCompilerVariables::MinBatchSize;
			const int32 NumJobs = Manager->AllJobs.GetPendingJobs(EShaderCompilerWorkerType::Distributed, Priority, MinBatchSize, INT32_MAX, PendingJobs);
			if (NumJobs > 0)
			{
				UE_LOGF(LogShaderCompilers, Verbose, "Started %d 'Distributed' shader compile jobs with '%ls' priority",
					NumJobs,
					ShaderCompileJobPriorityToString((EShaderCompileJobPriority)PriorityIndex));
			}
			if (PendingJobs.Num() >= DistributedShaderCompilerVariables::MinBatchSize)
			{
				break;
			}
		}
	}

	if (PendingJobs.Num() > 0)
	{
		// Increase the batch size when more jobs are queued/in flight.

		// Build farm is much more prone to pool oversubscription, so make sure the jobs are submitted in batches of at least MinBatchSize
		int MinJobsPerBatch = GIsBuildMachine ? DistributedShaderCompilerVariables::MinBatchSize : 1;

		// Just to provide typical numbers: the number of total jobs is usually in tens of thousands at most, oftentimes in low thousands. Thus JobsPerBatch when calculated as a log2 rarely reaches the value of 16,
		// and that seems to be a sweet spot: lowering it does not result in faster completion, while increasing the number of jobs per batch slows it down.
		const uint32 JobsPerBatch = FMath::Max(MinJobsPerBatch, FMath::FloorToInt(FMath::LogX(2.f, PendingJobs.Num() + NumDispatchedJobs)));
		UE_LOGF(LogShaderCompilers, Log, "Current jobs: %d, Batch size: %d, Num Already Dispatched: %d", PendingJobs.Num(), JobsPerBatch, NumDispatchedJobs);


		struct FJobBatch
		{
			TArray<FShaderCommonCompileJobPtr> Jobs;
			TSet<const FShaderType*> UniquePointers;

			bool operator == (const FJobBatch& B) const
			{
				return Jobs == B.Jobs;
			}
		};


		// Different batches.
		TArray<FJobBatch> JobBatches;


		for (int32 i = 0; i < PendingJobs.Num(); i++)
		{
			if (PendingJobs[i]->Priority > EShaderCompileJobPriority::High)
			{
				// Submit single job immediately if it has a higher priority than the default
				TArray<FShaderCommonCompileJobPtr> SingleJobArray{ PendingJobs[i] };
				DispatchShaderCompileJobsBatch(SingleJobArray);
				continue;
			}

			// Avoid to have multiple of permutation of same global shader in same batch, to avoid pending on long shader compilation
			// of batches that tries to compile permutation of a global shader type that is giving a hard time to the shader compiler.
			const FShaderType* OptionalUniqueShaderType = nullptr;
			if (FShaderCompileJob* ShaderCompileJob = PendingJobs[i]->GetSingleShaderJob())
			{
				if (ShaderCompileJob->Key.ShaderType->GetGlobalShaderType())
				{
					OptionalUniqueShaderType = ShaderCompileJob->Key.ShaderType;
				}
			}

			// Find a batch this compile job can be packed with.
			FJobBatch* SelectedJobBatch = nullptr;
			{
				if (JobBatches.Num() == 0)
				{
					SelectedJobBatch = &JobBatches[JobBatches.Emplace()];
				}
				else if (OptionalUniqueShaderType)
				{
					for (FJobBatch& PendingJobBatch : JobBatches)
					{
						if (!PendingJobBatch.UniquePointers.Contains(OptionalUniqueShaderType))
						{
							SelectedJobBatch = &PendingJobBatch;
							break;
						}
					}

					if (!SelectedJobBatch)
					{
						SelectedJobBatch = &JobBatches[JobBatches.Emplace()];
					}
				}
				else
				{
					SelectedJobBatch = &JobBatches[0];
				}
			}

			// Assign compile job to job batch.
			{
				SelectedJobBatch->Jobs.Add(PendingJobs[i]);
				if (OptionalUniqueShaderType)
				{
					SelectedJobBatch->UniquePointers.Add(OptionalUniqueShaderType);
				}
			}

			// Kick off compile job batch.
			if (SelectedJobBatch->Jobs.Num() == JobsPerBatch)
			{
				DispatchShaderCompileJobsBatch(SelectedJobBatch->Jobs);
				JobBatches.RemoveSingleSwap(*SelectedJobBatch);
			}
		}

		// Kick off remaining compile job batches.
		for (FJobBatch& PendingJobBatch : JobBatches)
		{
			DispatchShaderCompileJobsBatch(PendingJobBatch.Jobs);
		}
	}

	TMemoryHasher<FXxHash64Builder, FXxHash64> WorkerStateHasher;
	bool bHasAnyJobs = false;

	for (auto Iter = DispatchedTasks.CreateIterator(); Iter; ++Iter)
	{
		bool bOutputFileReadFailed = true;

		FDistributedShaderCompilerTask* Task = Iter->Get();

		bHasAnyJobs = bHasAnyJobs || !Task->ShaderJobs.IsEmpty();

		// Add jobs input hashes to the current state hash
		bool bIsTaskReady = Task->Future.IsReady();
		WorkerStateHasher << bIsTaskReady;
		for (const FShaderCommonCompileJobPtr& Job : Task->ShaderJobs)
		{
			WorkerStateHasher << Job->InputHash;
		}

		if (!bIsTaskReady)
		{
			continue;
		}

		FDistributedBuildTaskResult Result = Task->Future.Get();
		NumDispatchedJobs -= Task->ShaderJobs.Num();

		if (Result.ReturnCode != 0)
		{
			UE_LOGF(LogShaderCompilers, Error, "Shader compiler returned a non-zero error code (%d).", Result.ReturnCode);
		}


		if (Result.bCompleted)
		{
			// Check the output file exists. If it does, attempt to open it and serialize in the completed jobs.
			bool bCompileJobsSucceeded = false;

			TUniquePtr<FArchive> OutputFileAr;
			if (!Result.OutputData.IsEmpty())
			{
				OutputFileAr = TUniquePtr<FArchive>(new FShaderTaskFileMemoryReader(Result.OutputData));
			}
			else if (IFileManager::Get().FileExists(*Task->OutputFilePath))
			{
				OutputFileAr = TUniquePtr<FArchive>(IFileManager::Get().CreateFileReader(*Task->OutputFilePath, FILEREAD_Silent));
			}

			if (OutputFileAr.IsValid())
			{
				bOutputFileReadFailed = false;
				FShaderCompileWorkerDiagnostics WorkerDiagnostics;
				if (FShaderCompileWorkerUtil::ReadTasks(
					Task->ShaderJobs,
					*OutputFileAr,
					GShaderCompilerDumpWorkerDiagnostics ? &WorkerDiagnostics : nullptr,
					FShaderCompileWorkerUtil::EReadTasksFlags::WillRetry) == FSCWErrorCode::Success)
				{
					bCompileJobsSucceeded = true;

					if (GShaderCompilerDumpWorkerDiagnostics)
					{
						FString BatchLabel = FPaths::GetCleanFilename(Task->InputFilePath);
						constexpr uint32 UnavailableWorkerId = 0;
						GShaderCompilerStats->RegisterWorkerDiagnostics(WorkerDiagnostics, MoveTemp(BatchLabel), Task->ShaderJobs.Num(), UnavailableWorkerId);
					}
				}
			}
			
			if (!bCompileJobsSucceeded)
			{
				// Reading result from distributed job failed, so recompile shaders in current job batch locally
				UE_LOGF(LogShaderCompilers, Display, "Rescheduling shader compilation to run locally after distributed job failed: %ls", *Task->OutputFilePath);

				FString JobDiagnostics;
				for (int32 JobIndex = 0; JobIndex < Task->ShaderJobs.Num(); ++JobIndex)
				{
					FShaderCommonCompileJobPtr& Job = Task->ShaderJobs[JobIndex];

					// Rescheduling jobs after distributed readback failed should be rare, so display all job details with default verbosity
					JobDiagnostics.Empty();
					Job->AppendDiagnostics(JobDiagnostics, JobIndex, Task->ShaderJobs.Num());
					UE_LOGF(LogShaderCompilers, Display, "Executing %ls", *JobDiagnostics);

					// Dumping debug info in case the compile job crashes the cooker
					FShaderCompileInternalUtilities::DumpDebugInfo(*Job);

					Job->ForEachSingleShaderJob(
						[](FShaderCompileJob& SingleJob)
						{
							SingleJob.Output = FShaderCompilerOutput();
							SingleJob.UpdateStatus(EShaderCompileJobStatus::Pending, EShaderCompileJobCompletionMethod::LocalExecution);
						});

					FShaderCompileUtilities::ExecuteShaderCompileJob(*Job);
				}
			}

			// Enter the critical section so we can access the input and output queues
			{
				FScopeLock Lock(&Manager->CompileQueueSection);
				for (const auto& Job : Task->ShaderJobs)
				{
					Manager->ProcessFinishedJob(Job, EShaderCompileJobStatus::Completed);
				}
			}
		}
		else
		{
			// The compile job was canceled. Return the jobs to the manager's compile queue.
			UE_LOGF(LogShaderCompilers, Display, "Distributed build task did not complete; returning %d jobs to the compile queue", Task->ShaderJobs.Num());
			FScopeLock Lock(&Manager->CompileQueueSection);
			Manager->AllJobs.SubmitJobs(Task->ShaderJobs);
		}

		if (!CachedController.SupportsVirtualFiles())
		{
			// Delete input and output files, if they exist.
			while (!IFileManager::Get().Delete(*Task->InputFilePath, false, true, true))
			{
				FPlatformProcess::Sleep(0.01f);
			}

			if (!bOutputFileReadFailed)
			{
				while (!IFileManager::Get().Delete(*Task->OutputFilePath, false, true, true))
				{
					FPlatformProcess::Sleep(0.01f);
				}
			}
		}

		Iter.RemoveCurrent();
	}

	// Yield for a short while to stop this thread continuously polling the disk.
	FPlatformProcess::Sleep(0.01f);

	// Check if shader jobs have not changed in too long
	if (!WorkerStateHeartbeat(bHasAnyJobs ? WorkerStateHasher.Finalize().Hash : 0))
	{
		ReportShaderCompileWorkerDistributedDiagnostics(DispatchedTasks);
		TransferJobsToLocalWorker(MoveTemp(DispatchedTasks));

		UE_LOGF(LogShaderCompilers, Warning, "Shutting down distributed shader compiling thread due to heartbeat failure");

		Stop();
	}

	// Return true if there is more work to be done.
	return Manager->AllJobs.GetNumOutstandingJobs() > 0;
}

void FShaderCompileDistributedThreadRunnable_Interface::ForEachPendingJob(const FShaderCompileJobCallback& PendingJobCallback) const
{
	for (auto Iter = DispatchedTasks.CreateConstIterator(); Iter; ++Iter)
	{
		const FDistributedShaderCompilerTask* Task = Iter->Get();
		for (const FShaderCommonCompileJobPtr& Job : Task->ShaderJobs)
		{
			if (!PendingJobCallback(Job.GetReference()))
			{
				return;
			}
		}
	}
}

const TCHAR* FShaderCompileDistributedThreadRunnable_Interface::GetThreadName() const
{
	return TEXT("ShaderCompilingThread-Distributed");
}
