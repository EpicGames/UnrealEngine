// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	ShaderCompilerEditor.cpp:
	Platform independent shader compilations functions intended for WITH_EDITOR only.
=============================================================================*/

#if WITH_EDITOR

#include "ShaderCompilerPrivate.h"

#include "CoreMinimal.h"
#include "GlobalShader.h"
#include "Interfaces/IShaderFormat.h"
#include "Interfaces/ITargetPlatform.h"
#include "Interfaces/ITargetPlatformManagerModule.h"
#include "Logging/StructuredLog.h"
#include "MaterialShared.h"
#include "Materials/Material.h"
#include "Materials/MaterialInstance.h"
#include "ProfilingDebugging/CookStats.h"
#include "ProfilingDebugging/DiagnosticTable.h"
#include "ProfilingDebugging/LoadTimeTracker.h"
#include "ProfilingDebugging/StallDetector.h"
#include "Serialization/NameAsStringProxyArchive.h"
#include "ShaderDiagnostics.h"
#include "ShaderSerialization.h"
#include "UObject/Linker.h"
#include "UObject/Package.h"
#include "UObject/UObjectIterator.h"
#include "UnrealEngine.h"

#include "Serialization/ArchiveSavePackageDataBuffer.h"
#include "UObject/UObjectGlobals.h"
#include "DerivedDataCache.h"
#include "DerivedDataRequestOwner.h"
#include "TextureCompiler.h"
#include "Rendering/StaticLightingSystemInterface.h"
#include "DirectoryWatcherModule.h"
#include "IDirectoryWatcher.h"

#if WITH_ODSC
#include "ODSC/ODSCManager.h"
#include "UnrealEngine.h"
#endif


#if 0 // UNUSED
static FDelayedAutoRegisterHelper GKickOffShaderAutoGenForPlatforms(EDelayedRegisterRunPhase::DeviceProfileManagerReady, []
{
	// also do this for all active target platforms (e.g. when cooking)
	ITargetPlatformManagerModule* TPM = GetTargetPlatformManager();
	if (TPM)
	{
		const TArray<ITargetPlatform*>& Platforms = TPM->GetActiveTargetPlatforms();

		for (int32 Index = 0; Index < Platforms.Num(); ++Index)
		{
			TArray<FName> DesiredShaderFormats;
			checkf(Platforms[Index], TEXT("Null platform on the list of active platforms!"));
			Platforms[Index]->GetAllTargetedShaderFormats(DesiredShaderFormats);

			for (int32 FormatIndex = 0; FormatIndex < DesiredShaderFormats.Num(); ++FormatIndex)
			{
				FShaderCompileUtilities::GenerateBrdfHeaders(DesiredShaderFormats[FormatIndex]);
			}
		}
	}

	// also do this for the editor mobile preview
	EShaderPlatform MobilePreviewShaderPlatform = GShaderPlatformForFeatureLevel[ERHIFeatureLevel::ES3_1];
	if (MobilePreviewShaderPlatform != SP_NumPlatforms)
	{
		FShaderCompileUtilities::GenerateBrdfHeaders(MobilePreviewShaderPlatform);
	}
});
#endif


static void PrepareGlobalShaderCompileJob(EShaderPlatform Platform,
	EShaderPermutationFlags PermutationFlags,
	const FShaderPipelineType* ShaderPipeline,
	FShaderCompileJob* NewJob)
{
	const FShaderCompileJobKey& Key = NewJob->Key;
	const FGlobalShaderType* ShaderType = Key.ShaderType->AsGlobalShaderType();

	FShaderCompilerEnvironment& ShaderEnvironment = NewJob->Input.Environment;

	UE_LOGF(LogShaders, Verbose, "	%ls (permutation %d)", ShaderType->GetName(), Key.PermutationId);
	COOK_STAT(GlobalShaderCookStats::ShadersCompiled++);

	// Allow the shader type to modify the compile environment.
	ShaderType->SetupCompileEnvironment(Platform, Key.PermutationId, PermutationFlags, ShaderEnvironment);

	static FString GlobalName("Global");

	NewJob->bErrorsAreLikelyToBeCode = true;
	NewJob->bIsGlobalShader = true;
	NewJob->bIsDefaultMaterial = false;

	// Compile the shader environment passed in with the shader type's source code.
	::GlobalBeginCompileShader(
		GlobalName,
		nullptr,
		ShaderType,
		ShaderPipeline,
		Key.PermutationId,
		ShaderType->GetShaderFilename(),
		ShaderType->GetFunctionName(),
		FShaderTarget(ShaderType->GetFrequency(), Platform),
		NewJob->Input
	);
}

void FGlobalShaderTypeCompiler::BeginCompileShader(const FGlobalShaderType* ShaderType, int32 PermutationId, EShaderPlatform Platform, EShaderPermutationFlags PermutationFlags, TArray<FShaderCommonCompileJobPtr>& NewJobs)
{
	// Global shaders are always high priority (often need to block on completion)
	FShaderCompileJob* NewJob = GShaderCompilingManager->PrepareShaderCompileJob(GlobalShaderMapId, FShaderCompileJobKey(ShaderType, Platform, nullptr, PermutationId), EShaderCompileJobPriority::High);
	if (NewJob)
	{
		PrepareGlobalShaderCompileJob(Platform, PermutationFlags, nullptr, NewJob);
		NewJobs.Add(FShaderCommonCompileJobPtr(NewJob));
	}
}

void FGlobalShaderTypeCompiler::BeginCompileShaderPipeline(EShaderPlatform Platform, EShaderPermutationFlags PermutationFlags, const FShaderPipelineType* ShaderPipeline, TArray<FShaderCommonCompileJobPtr>& NewJobs)
{
	check(ShaderPipeline);
	UE_LOGF(LogShaders, Verbose, "	Pipeline: %ls", ShaderPipeline->GetName());

	// Add all the jobs as individual first, then add the dependencies into a pipeline job
	FShaderPipelineCompileJob* NewPipelineJob = GShaderCompilingManager->PreparePipelineCompileJob(GlobalShaderMapId, FShaderPipelineCompileJobKey(ShaderPipeline, Platform, nullptr, kUniqueShaderPermutationId), EShaderCompileJobPriority::High);
	if (NewPipelineJob)
	{
		for (FShaderCompileJob* StageJob : NewPipelineJob->StageJobs)
		{
			PrepareGlobalShaderCompileJob(Platform, PermutationFlags, ShaderPipeline, StageJob);
		}
		NewJobs.Add(FShaderCommonCompileJobPtr(NewPipelineJob));
	}
}

FShader* FGlobalShaderTypeCompiler::FinishCompileShader(const FGlobalShaderType* ShaderType, const FShaderCompileJob& CurrentJob, const FShaderPipelineType* ShaderPipelineType)
{
	FShader* Shader = nullptr;
	if (CurrentJob.bSucceeded)
	{
		EShaderPlatform Platform = CurrentJob.Input.Target.GetPlatform();
		FGlobalShaderMapSection* Section = GGlobalShaderMap[Platform]->FindOrAddSection(ShaderType);

		Section->GetResourceCode()->AddShaderCompilerOutput(CurrentJob.Output, CurrentJob.Key, CurrentJob.Input.GenerateDebugInfo());

		if (ShaderPipelineType && !ShaderPipelineType->ShouldOptimizeUnusedOutputs(CurrentJob.Input.Target.GetPlatform()))
		{
			// If sharing shaders in this pipeline, remove it from the type/id so it uses the one in the shared shadermap list
			ShaderPipelineType = nullptr;
		}

		// Create the global shader map hash
		uint64 GlobalShaderMapHash;
		{
			FStringView GlobalShaderString = TEXTVIEW("GlobalShaderMap");
			GlobalShaderMapHash = FXxHash64::HashBuffer(GlobalShaderString.GetData(), GlobalShaderString.NumBytes()).Hash;
		}

		Shader = ShaderType->ConstructCompiled(FGlobalShaderType::CompiledShaderInitializerType(ShaderType, nullptr, CurrentJob.Key.PermutationId, CurrentJob.Output, GlobalShaderMapHash, ShaderPipelineType, nullptr));
		CurrentJob.Output.ParameterMap.VerifyBindingsAreComplete(ShaderType->GetName(), CurrentJob.Output.Target, CurrentJob.Key.VFType);
	}

	return Shader;
}

FString GetGlobalShaderMapKeyString(const FGlobalShaderMapId& ShaderMapId, EShaderPlatform Platform, TArray<FShaderTypeDependency> const& Dependencies)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(GetGlobalShaderMapKeyString);
	FName Format = LegacyShaderPlatformToShaderFormat(Platform);
	FString ShaderMapKeyString = Format.ToString() + TEXT("_") + FString(FString::FromInt(GetTargetPlatformManagerRef().ShaderFormatVersion(Format))) + TEXT("_");
	ShaderMapAppendKeyString(Platform, ShaderMapKeyString);
	ShaderMapId.AppendKeyString(ShaderMapKeyString, Dependencies);

	const EShaderPermutationFlags PermutationFlags = ShaderMapId.GetShaderPermutationFlags();

	// Construct a hash of all the environment modifications applied for each shader type & permutation
	FMemoryHasherBlake3 Hasher;
	for (const FShaderTypeDependency& ShaderTypeDep : Dependencies)
	{
		const FGlobalShaderType* GlobalShaderType = FindShaderTypeByName(ShaderTypeDep.ShaderTypeName)->AsGlobalShaderType();
		for (int32 PermutationId = 0; PermutationId < GlobalShaderType->GetPermutationCount(); PermutationId++)
		{
			if (GlobalShaderType->ShouldCompilePermutation(Platform, PermutationId, PermutationFlags))
			{
				FShaderCompilerEnvironment Env(Hasher);
				GlobalShaderType->SetupCompileEnvironment(Platform, PermutationId, PermutationFlags, Env);
				Env.SerializeEverythingButFiles(Hasher);
			}
		}
	}

	// * 2 for hex representation of hash; + 6 for tag/underscores
	TStringBuilder<sizeof(TCHAR) * (sizeof(FBlake3Hash::ByteArray) * 2 + 6)> EnvHashString;
	EnvHashString << "_EMH_" << Hasher.Finalize() << "_";
	check(EnvHashString.GetAllocatedSize() == 0);
	ShaderMapKeyString.Append(EnvHashString.ToView());

	return FString::Printf(TEXT("%s_%s_%s"), TEXT("GSM"), *GetGlobalShaderMapDDCGuid().ToString(), *ShaderMapKeyString);
}

/** Creates a string key for the derived data cache entry for the global shader map. */
UE::DerivedData::FCacheKey GetGlobalShaderMapKey(const FGlobalShaderMapId& ShaderMapId, EShaderPlatform Platform, const ITargetPlatform* TargetPlatform, TArray<FShaderTypeDependency> const& Dependencies)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(GetGlobalShaderMapKey);

	const FString DataKey = GetGlobalShaderMapKeyString(ShaderMapId, Platform, Dependencies);
	static const UE::DerivedData::FCacheBucket Bucket(ANSITEXTVIEW("GlobalShaderMap"), TEXTVIEW("GlobalShader"));
	return {Bucket, FIoHash::HashBuffer(MakeMemoryView(FTCHARToUTF8(DataKey)))};
}

UE::FSharedString GetGlobalShaderMapName(const FGlobalShaderMapId& ShaderMapId, EShaderPlatform Platform, const FString& Key)
{
	return UE::FSharedString(WriteToString<256>(TEXTVIEW("GlobalShaderMap ["), LegacyShaderPlatformToShaderFormat(Platform), TEXTVIEW(", "), Key, TEXTVIEW("]")));
}

static void CompileGlobalShaderMapForRemote(
	const TArray<const FShaderType*>& OutdatedShaderTypes, 
	const TArray<const FShaderPipelineType*>& OutdatedShaderPipelineTypes, 
	const EShaderPlatform ShaderPlatform, 
	const ITargetPlatform* TargetPlatform,
	TArray<uint8>* OutArray,
	const FShaderCompilerFlags& InExtraCompilerFlags = {})
{
	UE_LOGF(LogShaders, Display, "Recompiling global shaders.");

	// Kick off global shader recompiles
	BeginRecompileGlobalShaders(OutdatedShaderTypes, OutdatedShaderPipelineTypes, ShaderPlatform, TargetPlatform, InExtraCompilerFlags);

	// Block on global shaders
	FinishRecompileGlobalShaders();

	// Write the shader compilation info to memory, converting FName to strings
	TOptional<FArchiveSavePackageDataBuffer> ArchiveSavePackageData;
	FMemoryWriter MemWriter(*OutArray, true);
	FNameAsStringProxyArchive Ar(MemWriter);

	if (TargetPlatform != nullptr)
	{
		ArchiveSavePackageData.Emplace(TargetPlatform);
		Ar.SetSavePackageData(&ArchiveSavePackageData.GetValue());
	}

	// save out the global shader map to the byte array
	SaveGlobalShadersForRemoteRecompile(Ar, ShaderPlatform);
}

static void SaveShaderMapsForRemote(ITargetPlatform* TargetPlatform, const TMap<FString, TArray<TRefCountPtr<FMaterialShaderMap>>>& CompiledShaderMaps, TArray<uint8>* OutArray)
{
	// write the shader compilation info to memory, converting fnames to strings
	TOptional<FArchiveSavePackageDataBuffer> ArchiveSavePackageData;
	FMemoryWriter MemWriter(*OutArray, true);
	FNameAsStringProxyArchive Ar(MemWriter);

	if (TargetPlatform != nullptr)
	{
		ArchiveSavePackageData.Emplace(TargetPlatform);
		Ar.SetSavePackageData(&ArchiveSavePackageData.GetValue());
	}

	// save out the shader map to the byte array
	FMaterialShaderMap::SaveForRemoteRecompile(Ar, CompiledShaderMaps);
}

FShaderRecompileData::FShaderRecompileData(const FString& InPlatformName, TArray<FString>* OutModifiedFiles, TArray<uint8>* OutMeshMaterialMaps, TArray<uint8>* OutGlobalShaderMap)
	: PlatformName(InPlatformName),
	ModifiedFiles(OutModifiedFiles),
	MeshMaterialMaps(OutMeshMaterialMaps),
	GlobalShaderMap(OutGlobalShaderMap)
{
}

FShaderRecompileData::FShaderRecompileData(const FString& InPlatformName, EShaderPlatform InShaderPlatform, ODSCRecompileCommand InCommandType, TArray<FString>* OutModifiedFiles, TArray<uint8>* OutMeshMaterialMaps, TArray<uint8>* OutGlobalShaderMap)
	: PlatformName(InPlatformName),
	ShaderPlatform(InShaderPlatform),
	ModifiedFiles(OutModifiedFiles),
	MeshMaterialMaps(OutMeshMaterialMaps),
	CommandType(InCommandType),
	GlobalShaderMap(OutGlobalShaderMap)
{
}

FArchive& operator<<(FArchive& Ar, FShaderRecompileData& RecompileData)
{

	int32 iShaderPlatform = static_cast<int32>(RecompileData.ShaderPlatform);
	int32 iFeatureLevel = static_cast<int32>(RecompileData.FeatureLevel);
	int32 iQualityLevel = static_cast<int32>(RecompileData.QualityLevel);

	Ar << RecompileData.MaterialsToLoad;
	Ar << RecompileData.ShaderTypesToLoad;
	Ar << RecompileData.ExtraCompilerFlags;
	Ar << iShaderPlatform;
	Ar << iFeatureLevel;
	Ar << iQualityLevel;
	Ar << RecompileData.CommandType;
	Ar << RecompileData.ShadersToRecompile;

	if (Ar.IsLoading())
	{
		RecompileData.ShaderPlatform = static_cast<EShaderPlatform>(iShaderPlatform);
		RecompileData.FeatureLevel = static_cast<ERHIFeatureLevel::Type>(iFeatureLevel);
		RecompileData.QualityLevel = static_cast<EMaterialQualityLevel::Type>(iQualityLevel);
	}

	return Ar;
}

void RecompileShadersForRemote(
	FShaderRecompileData& Args,
	const FString& OutputDirectory)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(RecompileShadersForRemote);

	// figure out what shader platforms to recompile
	ITargetPlatformManagerModule* TPM = GetTargetPlatformManager();
	ITargetPlatform* TargetPlatform = TPM->FindTargetPlatform(Args.PlatformName);
	if (TargetPlatform == nullptr)
	{
		UE_LOGF(LogShaders, Display, "Failed to find target platform module for %ls", *Args.PlatformName);
		return;
	}

	const double StartTime = FPlatformTime::Seconds();

	UE_LOGF(LogShaders, Display, "");
	UE_LOGF(LogShaders, Display, "********************************");
	UE_LOGF(LogShaders, Display, "Received compile shader request %ls.", ODSCCmdEnumToString(Args.CommandType));

	const bool bPreviousState = GShaderCompilingManager->IsShaderCompilationSkipped();
	GShaderCompilingManager->SkipShaderCompilation(false);

	TArray<FName> DesiredShaderFormats;
	TargetPlatform->GetAllTargetedShaderFormats(DesiredShaderFormats);

	UE_LOGF(LogShaders, Verbose, "Loading %d materials...", Args.MaterialsToLoad.Num());
	// make sure all materials the client has loaded will be processed
	TArray<UMaterialInterface*> MaterialsToCompile;

	for (int32 Index = 0; Index < Args.MaterialsToLoad.Num(); Index++)
	{
		UE_LOGF(LogShaders, Verbose, "   --> %ls", *Args.MaterialsToLoad[Index]);
		MaterialsToCompile.Add(LoadObject<UMaterialInterface>(nullptr, *Args.MaterialsToLoad[Index]));
	}

	UE_LOGF(LogShaders, Verbose, "  Done!");

	const uint32 StartTotalShadersCompiled = GShaderCompilerStats->GetTotalShadersCompiled();

	// Pick up new changes to shader files
	FlushShaderFileCache();

	// If we have an explicit list of shaders to compile from ODSC just compile those.
	if (Args.ShadersToRecompile.Num() && (Args.MeshMaterialMaps != nullptr))
	{
		TMap<FString, TArray<TRefCountPtr<FMaterialShaderMap>>> CompiledShaderMaps;
		{
			if (!GShaderCompilingManager->AllowForIncompleteShaderMaps())
			{
				UE_LOGF(LogShaders, Warning, "AllowForIncompleteShaderMaps is not set to true, ODSC shaders will most likely not be recompiled");
			}

			// UMaterial::CompileODSCMaterialsForRemoteRecompile will call LoadObjects on the material names but doesn't keep them around. Add a GC guard to ensure we can still get them
			// before they get unloaded, so that the whole chain UMaterial->FMaterial->FMaterialShaderMap is kept intact, and we can merge the next batch of ODSC requests
			FGCScopeGuard NoGCScopeGuard;
			UMaterial::CompileODSCMaterialsForRemoteRecompile(Args.ShadersToRecompile, CompiledShaderMaps, Args.ODSCCustomLoadMaterial);
			if (Args.LoadedMaterialsToRecompile)
			{
				for (auto Iter : CompiledShaderMaps)
				{
					TStrongObjectPtr<UMaterialInterface> MaterialInterface;
					
					if (Args.ODSCCustomLoadMaterial)
					{
						MaterialInterface = TStrongObjectPtr<UMaterialInterface>(Args.ODSCCustomLoadMaterial(Iter.Key));
					}
					else
					{
						MaterialInterface = TStrongObjectPtr<UMaterialInterface>(FindObject<UMaterialInterface>(nullptr, *Iter.Key));
					}

					if (MaterialInterface)
					{
						ResetLoaders(MaterialInterface->GetPackage());
						Args.LoadedMaterialsToRecompile->Add(MaterialInterface);
					}
					else
					{
						UE_LOGF(LogShaders, Warning, "Failed to find Material %ls. Reloading on the client will be skipped", *Iter.Key);
					}
				}
			}
		}
		SaveShaderMapsForRemote(TargetPlatform, CompiledShaderMaps, Args.MeshMaterialMaps);
	}
	else
	{
		for (int32 FormatIndex = 0; FormatIndex < DesiredShaderFormats.Num(); FormatIndex++)
		{
			// get the shader platform enum
			const EShaderPlatform ShaderPlatform = ShaderFormatToLegacyShaderPlatform(DesiredShaderFormats[FormatIndex]);

			// Only compile for the desired platform if requested
			if (ShaderPlatform == Args.ShaderPlatform || Args.ShaderPlatform == SP_NumPlatforms)
			{
				if (Args.CommandType == ODSCRecompileCommand::SingleShader &&
					Args.ShaderTypesToLoad.Len() > 0)
				{
					constexpr bool bSearchAsRegexFilter = true;
					TArray<const FShaderType*> ShaderTypes = FShaderType::GetShaderTypesByFilename(*Args.ShaderTypesToLoad, bSearchAsRegexFilter);
					TArray<const FShaderPipelineType*> ShaderPipelineTypes = FShaderPipelineType::GetShaderPipelineTypesByFilename(*Args.ShaderTypesToLoad, bSearchAsRegexFilter);

					for (const FShaderType* ShaderType : ShaderTypes)
					{
						UE_LOGF(LogShaders, Display, "\t%ls...", ShaderType->GetName());
					}

					UpdateReferencedUniformBufferNames(ShaderTypes, {}, ShaderPipelineTypes);

					CompileGlobalShaderMapForRemote(ShaderTypes, ShaderPipelineTypes, ShaderPlatform, TargetPlatform, Args.GlobalShaderMap, Args.ExtraCompilerFlags);
				}
				else if (Args.CommandType == ODSCRecompileCommand::Global ||
						 Args.CommandType == ODSCRecompileCommand::Changed)
				{
					// figure out which shaders are out of date
					TArray<const FShaderType*> OutdatedShaderTypes;
					TArray<const FVertexFactoryType*> OutdatedFactoryTypes;
					TArray<const FShaderPipelineType*> OutdatedShaderPipelineTypes;

					// Explicitly get outdated types for global shaders.
					const FGlobalShaderMap* ShaderMap = GGlobalShaderMap[ShaderPlatform];
					if (ShaderMap)
					{
						ShaderMap->GetOutdatedTypes(OutdatedShaderTypes, OutdatedShaderPipelineTypes, OutdatedFactoryTypes);
					}

					UE_LOGF(LogShaders, Display, "\tFound %d outdated shader types.", OutdatedShaderTypes.Num() + OutdatedShaderPipelineTypes.Num());

					UpdateReferencedUniformBufferNames(OutdatedShaderTypes, OutdatedFactoryTypes, OutdatedShaderPipelineTypes);

					CompileGlobalShaderMapForRemote(OutdatedShaderTypes, OutdatedShaderPipelineTypes, ShaderPlatform, TargetPlatform, Args.GlobalShaderMap, Args.ExtraCompilerFlags);
				}

				// we only want to actually compile mesh shaders if a client directly requested it
				if ((Args.CommandType == ODSCRecompileCommand::Material || Args.CommandType == ODSCRecompileCommand::Changed) &&
					Args.MeshMaterialMaps != nullptr)
				{
					TMap<FString, TArray<TRefCountPtr<FMaterialShaderMap>>> CompiledShaderMaps;
					UMaterial::CompileMaterialsForRemoteRecompile(MaterialsToCompile, ShaderPlatform, TargetPlatform, CompiledShaderMaps);
					SaveShaderMapsForRemote(TargetPlatform, CompiledShaderMaps, Args.MeshMaterialMaps);
				}

				// save it out so the client can get it (and it's up to date next time), if we were sent a OutputDirectory to put it in
				FString GlobalShaderFilename;
				if (!OutputDirectory.IsEmpty())
				{
					GlobalShaderFilename = SaveGlobalShaderFile(ShaderPlatform, OutputDirectory, TargetPlatform);
				}

				// add this to the list of files to tell the other end about
				if (Args.ModifiedFiles && !GlobalShaderFilename.IsEmpty())
				{
					// need to put it in non-sandbox terms
					FString SandboxPath(GlobalShaderFilename);
					check(SandboxPath.StartsWith(OutputDirectory));
					SandboxPath.ReplaceInline(*OutputDirectory, TEXT("../../../"));
					FPaths::NormalizeFilename(SandboxPath);
					Args.ModifiedFiles->Add(SandboxPath);
				}
			}
		}
	}

	for (UMaterialInterface* MaterialInterface : MaterialsToCompile)
	{
		if (MaterialInterface)
		{
			ResetLoaders(MaterialInterface->GetPackage());
		}
	}
	GEngine->ForceGarbageCollection(true);

	UE_LOGF(LogShaders, Display, "");
	UE_LOGF(LogShaders, Display, "Compiled %u shaders in %.2f seconds.", GShaderCompilerStats->GetTotalShadersCompiled() - StartTotalShadersCompiled, FPlatformTime::Seconds() - StartTime);

	// Restore compilation state.
	GShaderCompilingManager->SkipShaderCompilation(bPreviousState);
}

void ShutdownShaderCompilers(TConstArrayView<const ITargetPlatform*> TargetPlatforms)
{
	ITargetPlatformManagerModule& PlatformManager = GetTargetPlatformManagerRef();
	for (const ITargetPlatform* TargetPlatform : TargetPlatforms)
	{
		TArray<FName> DesiredShaderFormats;
		TargetPlatform->GetAllTargetedShaderFormats(DesiredShaderFormats);
		for (FName FormatName : DesiredShaderFormats)
		{
			const IShaderFormat* ShaderFormat = PlatformManager.FindShaderFormat(FormatName);
			if (ShaderFormat)
			{
				ShaderFormat->NotifyShaderCompilersShutdown(FormatName);
			}
		}
	}
}

static inline FShader* ProcessCompiledJob(FShaderCompileJob* SingleJob, const FShaderPipelineType* Pipeline, TArray<EShaderPlatform>& ShaderPlatformsProcessed, TArray<const FShaderPipelineType*>& OutSharedPipelines)
{
	const FGlobalShaderType* GlobalShaderType = SingleJob->Key.ShaderType->GetGlobalShaderType();
	check(GlobalShaderType);
	FShader* Shader = FGlobalShaderTypeCompiler::FinishCompileShader(GlobalShaderType, *SingleJob, Pipeline);
	if (Shader)
	{
		// Add the new global shader instance to the global shader map if it's a shared shader
		EShaderPlatform Platform = (EShaderPlatform)SingleJob->Input.Target.Platform;
		if (!Pipeline || !Pipeline->ShouldOptimizeUnusedOutputs(Platform))
		{
			Shader = GGlobalShaderMap[Platform]->FindOrAddShader(GlobalShaderType, SingleJob->Key.PermutationId, Shader);
			// Add this shared pipeline to the list
			if (!Pipeline)
			{
				auto* JobSharedPipelines = SingleJob->SharingPipelines.Find(nullptr);
				if (JobSharedPipelines)
				{
					for (auto* SharedPipeline : *JobSharedPipelines)
					{
						OutSharedPipelines.AddUnique(SharedPipeline);
					}
				}
			}
		}
		ShaderPlatformsProcessed.AddUnique(Platform);
	}

	return Shader;
};

/** Saves the platform's shader map to the DDC. It is assumed that the caller will check IsComplete() first before calling the function. */
static void SaveGlobalShaderMapToDerivedDataCache(EShaderPlatform Platform)
{
	// We've finally built the global shader map, so we can count the miss as we put it in the DDC.
	COOK_STAT(auto Timer = GlobalShaderCookStats::UsageStats.TimeSyncWork());

	const ITargetPlatform* TargetPlatform = GGlobalShaderTargetPlatform[Platform];
	FShaderCacheSaveContext Ctx;
	FGlobalShaderMapId ShaderMapId(Platform, TargetPlatform);
	// caller should prevent incomplete shadermaps to be saved
	FGlobalShaderMap* GlobalSM = GetGlobalShaderMap(Platform);
	for (auto const& ShaderFilenameDependencies : ShaderMapId.GetShaderFilenameToDependeciesMap())
	{
		FGlobalShaderMapSection* Section = GlobalSM->FindSection(ShaderFilenameDependencies.Key);
		if (Section)
		{
			Section->FinalizeContent();

			// reuse serialize context, internal allocations will be kept so this minimizes heap alloc churn
			Ctx.Reset();

			Section->Serialize(Ctx);
			COOK_STAT(Timer.AddMiss(Ctx.GetSerializedSize()));

			using namespace UE::DerivedData;
			UE::FSharedString Name = GetGlobalShaderMapName(ShaderMapId, Platform, ShaderFilenameDependencies.Key);
			FCacheKey Key = GetGlobalShaderMapKey(ShaderMapId, Platform, TargetPlatform, ShaderFilenameDependencies.Value);

			FRequestOwner AsyncOwner(EPriority::Normal);
			FRequestBarrier AsyncBarrier(AsyncOwner);
			GetCache().Put({ { Name, Ctx.BuildCacheRecord(Key) } }, AsyncOwner);
			AsyncOwner.KeepAlive();
		}
	}
}

void ProcessCompiledGlobalShaders(const TArray<FShaderCommonCompileJobPtr>& CompilationResults)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(ProcessCompiledGlobalShaders);

	UE_LOGF(LogShaders, Verbose, "Compiled %u global shaders", CompilationResults.Num());

	FShaderDiagnosticInfo ShaderDiagInfo(CompilationResults);



	const int32 UniqueErrorCount = ShaderDiagInfo.UniqueErrors.Num();
	if (UniqueErrorCount)
	{
		// Report unique errors for global shaders.
		FString AllUniqueErrors = FString::Join(ShaderDiagInfo.UniqueErrors, TEXT("\n"));

		const TCHAR* RetryMsg = TEXT("\nEnable 'r.ShaderDevelopmentMode' in ConsoleVariables.ini for retries.");
		if (AreShaderErrorsFatal())
		{
			UE_LOGFMT_NSLOC(LogShaders, Fatal, "Shaders", "GlobalShadersCompilationFailed", "{NumErrors} errors encountered compiling global shaders for platform {Platform}:\n{Errors}{RetryMsg}",
				("NumErrors", UniqueErrorCount),
				("Platform", ShaderDiagInfo.TargetShaderPlatformString),
				("RetryMsg", IsRunningCommandlet() ? TEXT("") : RetryMsg),
				("Errors", AllUniqueErrors)
			);
		}
		else
		{
			UE_LOGFMT_NSLOC(LogShaders, Error, "Shaders", "GlobalShadersCompilationFailed", "{NumErrors} errors encountered compiling global shaders for platform {Platform}:\n{Errors}{RetryMsg}",
				("NumErrors", UniqueErrorCount),
				("Platform", ShaderDiagInfo.TargetShaderPlatformString),
				("RetryMsg", IsRunningCommandlet() ? TEXT("") : RetryMsg),
				("Errors", AllUniqueErrors)
			);
		}
	}

	for (const FString& WarningString : ShaderDiagInfo.UniqueWarnings)
	{
		if (ShouldShowShaderWarnings())
		{
			UE_LOGFMT_NSLOC(LogShaders, Warning, "Shaders", "GlobalShaderCompileWarning", "{WarningMessage}", ("WarningMessage", WarningString));
		}
		else
		{
			UE_LOGFMT_NSLOC(LogShaders, Display, "Shaders", "GlobalShaderCompileWarning", "{WarningMessage}", ("WarningMessage", WarningString));
		}
	}

	TArray<EShaderPlatform> ShaderPlatformsProcessed;
	TArray<const FShaderPipelineType*> SharedPipelines;

	for (int32 ResultIndex = 0; ResultIndex < CompilationResults.Num(); ResultIndex++)
	{
		const FShaderCommonCompileJob& CurrentJob = *CompilationResults[ResultIndex];
		FShaderCompileJob* SingleJob = nullptr;
		if ((SingleJob = (FShaderCompileJob*)CurrentJob.GetSingleShaderJob()) != nullptr)
		{
			ProcessCompiledJob(SingleJob, nullptr, ShaderPlatformsProcessed, SharedPipelines);
		}
		else
		{
			const auto* PipelineJob = CurrentJob.GetShaderPipelineJob();
			check(PipelineJob);

			FShaderPipeline* ShaderPipeline = new FShaderPipeline(PipelineJob->Key.ShaderPipeline);
			for (int32 Index = 0; Index < PipelineJob->StageJobs.Num(); ++Index)
			{
				SingleJob = PipelineJob->StageJobs[Index]->GetSingleShaderJob();
				FShader* Shader = ProcessCompiledJob(SingleJob, PipelineJob->Key.ShaderPipeline, ShaderPlatformsProcessed, SharedPipelines);
				ShaderPipeline->AddShader(Shader, SingleJob->Key.PermutationId);
			}
			ShaderPipeline->Validate(PipelineJob->Key.ShaderPipeline);

			EShaderPlatform Platform = (EShaderPlatform)PipelineJob->StageJobs[0]->GetSingleShaderJob()->Input.Target.Platform;
			check(ShaderPipeline && !GGlobalShaderMap[Platform]->HasShaderPipeline(PipelineJob->Key.ShaderPipeline));
			GGlobalShaderMap[Platform]->FindOrAddShaderPipeline(PipelineJob->Key.ShaderPipeline, ShaderPipeline);
		}
	}

	for (int32 PlatformIndex = 0; PlatformIndex < ShaderPlatformsProcessed.Num(); PlatformIndex++)
	{
		EShaderPlatform Platform = ShaderPlatformsProcessed[PlatformIndex];
		FGlobalShaderMap* GlobalShaderMap = GGlobalShaderMap[Platform];
		const ITargetPlatform* TargetPlatform = GGlobalShaderTargetPlatform[Platform];

		// Process the shader pipelines that share shaders
		FPlatformTypeLayoutParameters LayoutParams;
		LayoutParams.InitializeForPlatform(TargetPlatform);
		const EShaderPermutationFlags PermutationFlags = GetShaderPermutationFlags(LayoutParams);

		for (const FShaderPipelineType* ShaderPipelineType : SharedPipelines)
		{
			check(ShaderPipelineType->IsGlobalTypePipeline());
			if (!GlobalShaderMap->HasShaderPipeline(ShaderPipelineType))
			{
				auto& StageTypes = ShaderPipelineType->GetStages();

				FShaderPipeline* ShaderPipeline = new FShaderPipeline(ShaderPipelineType);
				for (int32 Index = 0; Index < StageTypes.Num(); ++Index)
				{
					FGlobalShaderType* GlobalShaderType = ((FShaderType*)(StageTypes[Index]))->GetGlobalShaderType();
					if (GlobalShaderType->ShouldCompilePermutation(Platform, kUniqueShaderPermutationId, PermutationFlags))
					{
						TShaderRef<FShader> Shader = GlobalShaderMap->GetShader(GlobalShaderType, kUniqueShaderPermutationId);
						check(Shader.IsValid());
						ShaderPipeline->AddShader(Shader.GetShader(), kUniqueShaderPermutationId);
					}
					else
					{
						break;
					}
				}
				ShaderPipeline->Validate(ShaderPipelineType);
				GlobalShaderMap->FindOrAddShaderPipeline(ShaderPipelineType, ShaderPipeline);
			}
		}

		// at this point the new global sm is populated and we can delete the deferred copy, if any
		delete GGlobalShaderMap_DeferredDeleteCopy[ShaderPlatformsProcessed[PlatformIndex]];	// even if it was nullptr, deleting null is Okay
		GGlobalShaderMap_DeferredDeleteCopy[ShaderPlatformsProcessed[PlatformIndex]] = nullptr;

		// Save the global shader map for any platforms that were recompiled, but only if it is complete (it can be also a subject to ODSC, perhaps unnecessarily, as we cannot use a partial global SM)
		FGlobalShaderMapId ShaderMapId(Platform, TargetPlatform);
		if (GlobalShaderMap->IsComplete(TargetPlatform))
		{
			SaveGlobalShaderMapToDerivedDataCache(ShaderPlatformsProcessed[PlatformIndex]);

			if (!GRHISupportsMultithreadedShaderCreation && Platform == GMaxRHIShaderPlatform)
			{
				ENQUEUE_RENDER_COMMAND(CreateRecursiveShaders)([](FRHICommandListImmediate&)
				{
					CreateRecursiveShaders();
				});
			}
		}
	}
}

void SaveGlobalShadersForRemoteRecompile(FArchive& Ar, EShaderPlatform ShaderPlatform)
{
	FGlobalShaderMap* GlobalShaderMap = GetGlobalShaderMap(ShaderPlatform);
	uint8 bIsValid = GlobalShaderMap != nullptr;
	Ar << bIsValid;

	if (GlobalShaderMap)
	{
		GlobalShaderMap->SaveToGlobalArchive(Ar);
	}
}

#endif // WITH_EDITOR
