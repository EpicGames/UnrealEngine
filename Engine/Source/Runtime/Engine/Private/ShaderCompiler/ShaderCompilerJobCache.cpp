// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	ShaderCompilerJobCache.cpp:
	Implements FShaderCompileJobCollection as well as internal class FShaderJobCache.
=============================================================================*/

#include "ShaderCompilerPrivate.h"

#include "DerivedDataCache.h"
#include "DerivedDataRequestOwner.h"
#include "Math/UnitConversion.h"
#include "Misc/FileHelper.h"
#include "ODSC/ODSCManager.h"
#include "ProfilingDebugging/CookStats.h"
#include "ProfilingDebugging/DiagnosticTable.h"
#include "ProfilingDebugging/LoadTimeTracker.h"
#include "ProfilingDebugging/StallDetector.h"
#include "Serialization/MemoryReader.h"
#include "Serialization/MemoryWriter.h"
#include "ShaderSerialization.h"


// Switch to Verbose after initial testing
#define UE_SHADERCACHE_LOG_LEVEL		VeryVerbose

// whether to parallelize writing/reading task files
#define UE_SHADERCOMPILER_FIFO_JOB_EXECUTION  1


static TAutoConsoleVariable<bool> CVarShaderCompilerDebugValidateJobCache(
	TEXT("r.ShaderCompiler.DebugValidateJobCache"),
	false,
	TEXT("Enables debug mode for job cache which will fully execute all jobs and validate that job outputs with matching input hashes match."),
	ECVF_Default);

int32 GShaderCompilerMaxJobCacheMemoryPercent = 5;
static FAutoConsoleVariableRef CVarShaderCompilerMaxJobCacheMemoryPercent(
	TEXT("r.ShaderCompiler.MaxJobCacheMemoryPercent"),
	GShaderCompilerMaxJobCacheMemoryPercent,
	TEXT("if != 0, shader compiler cache will be limited to this percentage of available physical RAM (5% by default). If 0, the usage will be unlimited. Minimum of this or r.ShaderCompiler.MaxJobCacheMemoryMB applies."),
	ECVF_Default);

int32 GShaderCompilerMaxJobCacheMemoryMB = 16LL * 1024LL;
static FAutoConsoleVariableRef CVarShaderCompilerMaxJobCacheMemoryMB(
	TEXT("r.ShaderCompiler.MaxJobCacheMemoryMB"),
	GShaderCompilerMaxJobCacheMemoryMB,
	TEXT("if != 0, shader compiler cache will be limited to this many megabytes (16GB by default). If 0, the usage will be unlimited. Minimum of this or r.ShaderCompiler.MaxJobCacheMemoryPercent applies."),
	ECVF_Default);

static TAutoConsoleVariable<bool> CVarShaderCompilerPerShaderDDCCook(
	TEXT("r.ShaderCompiler.PerShaderDDCCook"),
	true,
	TEXT("If true, per-shader DDC caching will be enabled during cooks."),
	ECVF_Default);

int32 GShaderCompilerPerShaderDDCGlobal = 1;
static FAutoConsoleVariableRef CVarShaderCompilerPerShaderDDCGlobal(
	TEXT("r.ShaderCompiler.PerShaderDDCGlobal"),
	GShaderCompilerPerShaderDDCGlobal,
	TEXT("if != 0, Per-shader DDC queries enabled for global and default shaders."),
	ECVF_Default);

int32 GShaderCompilerDebugStallSubmitJob = 0;
static FAutoConsoleVariableRef CVarShaderCompilerDebugStallSubmitJob(
	TEXT("r.ShaderCompiler.DebugStallSubmitJob"),
	GShaderCompilerDebugStallSubmitJob,
	TEXT("For debugging, a value in milliseconds to stall in SubmitJob, to help reproduce threading bugs."),
	ECVF_Default);

static TAutoConsoleVariable<bool> CVarShaderCompilerPerShaderDDCAsync(
	TEXT("r.ShaderCompiler.PerShaderDDCAsync"),
	true,
	TEXT("if != 0, Per-shader DDC queries will run async, instead of in the SubmitJobs task."),
	ECVF_Default);

static TAutoConsoleVariable<bool> CVarJobCacheDDC(
	TEXT("r.ShaderCompiler.JobCacheDDC"),
	true,
	TEXT("Skips compilation of all shaders on Material and Material Instance PostLoad and relies on on-demand shader compilation to compile what is needed."),
	ECVF_ReadOnly);

static TAutoConsoleVariable<bool> CVarJobCacheDDCPolicy(
	TEXT("r.ShaderCompiler.JobCacheDDCEnableRemotePolicy"),
	true,
	TEXT("If true, individual shader jobs will be cached to remote/shared DDC instances in all operation modes; if false they will only cache to DDC instances on the local machine.\n"),
	ECVF_ReadOnly);

static TAutoConsoleVariable<bool> CVarJobCacheDDCCookPolicy(
	TEXT("r.ShaderCompiler.JobCacheDDCCookEnableRemotePolicy"),
	true,
	TEXT("If true, individual shader jobs will be cached to remote/shared DDC instances in all cook commandlet only; if false they will only cache to DDC instances on the local machine.\n"),
	ECVF_ReadOnly);

int32 GShaderCompilerJobCacheOverflowReducePercent = 80;
static FAutoConsoleVariableRef CVarShaderCompilerJobCacheOverflowReducePercent(
	TEXT("r.ShaderCompiler.JobCacheOverflowReducePercent"),
	GShaderCompilerJobCacheOverflowReducePercent,
	TEXT("When shader compiler job cache memory overflows, reduce memory to this percentage of the maximum.  Reduces overhead relative to cleaning up items one at a time when at max budget."),
	ECVF_Default);

int32 GShaderCompilerDebugStallDDCQuery = 0;
static FAutoConsoleVariableRef CVarShaderCompilerDebugStallDCCQuery(
	TEXT("r.ShaderCompiler.DebugStallDDCQuery"),
	GShaderCompilerDebugStallDDCQuery,
	TEXT("For debugging, a value in milliseconds to stall in the DDC completion callback, to help reproduce threading bugs, or simulate higher latency DDC for perf testing."),
	ECVF_Default);

static TAutoConsoleVariable<bool> CVarDumpShaderOutputCacheHits(
	TEXT("r.DumpShaderOutputCacheHits"),
	false,
	TEXT("Dumps shader output bytecode and cache hits with reference to original output.\n")
	TEXT("Dumping shader output bytecode for all compile shaders also requires CVar r.DumpShaderDebugInfo=1."),
	ECVF_ReadOnly);

int32 GShaderCompilerDebugDiscardCacheOutputs = 0;
static FAutoConsoleVariableRef CVarShaderCompilerDebugDiscardCacheOutputs(
	TEXT("r.ShaderCompiler.DebugDiscardCacheOutputs"),
	GShaderCompilerDebugDiscardCacheOutputs,
	TEXT("if != 0, cache outputs are discarded (not added to the output map) for debugging purposes.\nEliminates usefulness of the cache, but allows repeated triggering of the same jobs for stress testing (for example, rapid undo/redo in the Material editor)."),
	ECVF_Default);


TRACE_DECLARE_ATOMIC_INT_COUNTER(Shaders_JobCacheSearchAttempts, TEXT("Shaders/JobCache/SearchAttempts"));
TRACE_DECLARE_ATOMIC_INT_COUNTER(Shaders_JobCacheHits, TEXT("Shaders/JobCache/Hits"));

TRACE_DECLARE_ATOMIC_INT_COUNTER(Shaders_JobCacheDDCRequests, TEXT("Shaders/JobCache/DDCRequests"));
TRACE_DECLARE_ATOMIC_INT_COUNTER(Shaders_JobCacheDDCHits, TEXT("Shaders/JobCache/DDCHits"));
TRACE_DECLARE_ATOMIC_MEMORY_COUNTER(Shaders_JobCacheDDCBytesReceived, TEXT("Shaders/JobCache/DDCBytesReceived"));
TRACE_DECLARE_ATOMIC_MEMORY_COUNTER(Shaders_JobCacheDDCBytesSent, TEXT("Shaders/JobCache/DDCBytesSent"));

#if WITH_EDITOR
namespace
{
	/** The FCacheBucket used with the DDC, cached to avoid recreating it for each request */
	UE::DerivedData::FCacheBucket ShaderJobCacheDDCBucket = UE::DerivedData::FCacheBucket(ANSITEXTVIEW("FShaderJobCacheShaders"), TEXTVIEW("Shader"));
}
#endif


/** Helper functions for logging more debug info */
namespace ShaderCompiler
{
	bool IsJobCacheDebugValidateEnabled()
	{
		return CVarShaderCompilerDebugValidateJobCache.GetValueOnAnyThread();
	}
} // namespace ShaderCompiler


/** The ODSC server is a special cook commandlet where we don't want to use the material map DDC. */
static bool IsRunningCookCommandletAndNotODSCServer()
{
	static const bool bIsCookCommandlet = IsRunningCookCommandlet();
	static const bool bIsODSCServer = FParse::Param(FCommandLine::Get(), TEXT("odsc"));
	return bIsCookCommandlet && !bIsODSCServer;
}

bool IsShaderJobCacheDDCEnabled()
{
#if WITH_EDITOR
	static const bool bForceAllowShaderCompilerJobCache = FParse::Param(FCommandLine::Get(), TEXT("forceAllowShaderCompilerJobCache"));
	static const bool bEnablePerShaderDDCCook = IsRunningCookCommandlet() && CVarShaderCompilerPerShaderDDCCook.GetValueOnAnyThread();
	static const bool bIsNonCookCommandlet = IsRunningCommandlet() && !IsRunningCookCommandlet();
#else
	const bool bForceAllowShaderCompilerJobCache = false;
	const bool bEnablePerShaderDDCCook = false;
	const bool bIsNonCookCommandlet = false;
#endif

	// Enable remote per-shader DDC for editor, game, cooks (if cvar is set), and for other commandlets only if the force flag is set on the cmdline.
	if ((GIsEditor || IsRunningGame() || bEnablePerShaderDDCCook) && (!bIsNonCookCommandlet || bForceAllowShaderCompilerJobCache))
	{
		// job cache itself must be enabled first
		return CVarJobCacheDDC.GetValueOnAnyThread();
	}

	return false;
}

bool IsMaterialMapDDCEnabled()
{
	// If we are loading individual shaders from the shader job cache for ODSC, don't attempt to load full material maps. 
	// Otherwise always load/cache material maps in cooks.
	return (IsShaderJobCacheDDCEnabled() == false) || IsRunningCookCommandletAndNotODSCServer();
}

bool ShouldCompileODSCOnlyShaders()
{
#if WITH_EDITOR
	static const bool bIsODSCEditor = !IsMaterialMapDDCEnabled();
	return bIsODSCEditor;
#elif WITH_ODSC
	static const bool bIsODSCClient = FODSCManager::IsODSCActive();
	return bIsODSCClient;
#else
	return false;
#endif
}

static bool IsShaderJobCacheDDCRemotePolicyEnabled()
{
	return CVarJobCacheDDCPolicy.GetValueOnAnyThread() || (IsRunningCookCommandlet() && CVarJobCacheDDCCookPolicy.GetValueOnAnyThread());
}

/** Copy of TIntrusiveLinkedListIterator, specific to FShaderCommonCompileJob */
class FShaderCommonCompileJobIterator
{
public:
	explicit FShaderCommonCompileJobIterator(FShaderCommonCompileJob* FirstLink)
		: CurrentLink(FirstLink)
	{ }

	/**
	 * Advances the iterator to the next element.
	 */
	FORCEINLINE void Next()
	{
		checkSlow(CurrentLink);
		CurrentLink = CurrentLink->NextLink;
	}

	FORCEINLINE FShaderCommonCompileJobIterator& operator++()
	{
		Next();
		return *this;
	}

	FORCEINLINE FShaderCommonCompileJobIterator operator++(int)
	{
		auto Tmp = *this;
		Next();
		return Tmp;
	}

	/** conversion to "bool" returning true if the iterator is valid. */
	FORCEINLINE explicit operator bool() const
	{
		return CurrentLink != nullptr;
	}

	FORCEINLINE bool operator==(const FShaderCommonCompileJobIterator& Rhs) const { return CurrentLink == Rhs.CurrentLink; }
	FORCEINLINE bool operator!=(const FShaderCommonCompileJobIterator& Rhs) const { return CurrentLink != Rhs.CurrentLink; }

	// Accessors.
	FORCEINLINE FShaderCommonCompileJob& operator->() const
	{
		checkSlow(this->CurrentLink);
		return *(this->CurrentLink);
	}

	FORCEINLINE FShaderCommonCompileJob& operator*() const
	{
		checkSlow(this->CurrentLink);
		return *(this->CurrentLink);
	}

private:
	FShaderCommonCompileJob* CurrentLink;
};

/** Map element type for job cache */
struct FShaderJobData
{
	using FJobOutputHash = FBlake3Hash;

	FShaderCompilerInputHash InputHash;

	/** Output hash will be zero if output data has not been written yet, or can be cleared if output data has been removed */
	FJobOutputHash OutputHash;

	/** Track which code path wrote this output, for tracking down a bug */
	bool bOutputFromDDC;

	/**
	 * In-flight job with the given input hash.  Needs to be a reference pointer to handle cancelling of jobs, where an async DDC query
	 * (which receives a pointer to FShaderJobData) may be in-flight that still references a job that has otherwise been deleted.
	 * Cancelled jobs will have been unlinked from the PendingSubmitJobTaskJobs list in RemoveAllPendingJobsWithId, which can be
	 * detected in the callback, and further processing on the job skipped.
	 */
	FShaderCommonCompileJobPtr JobInFlight;

	void ResetJobInFlight()
	{
		if (JobInFlight)
		{
#if WITH_EDITOR
			if (JobInFlight->RequestOwner.IsValid())
			{
				JobInFlight->RequestOwner->KeepAlive();
			}
#endif
			JobInFlight = nullptr;
		}
	}

	/** Head of a linked list of duplicate jobs */
	FShaderCommonCompileJob* DuplicateJobsWaitList = nullptr;

	bool IsEmpty() const
	{
		return OutputHash.IsZero() && JobInFlight == nullptr && DuplicateJobsWaitList == nullptr;
	}

	FORCEINLINE bool HasOutput() const
	{
		return OutputHash.IsZero() == false;
	}
};

/** Block of map elements for job cache */
struct FShaderJobDataBlock
{
	static const int32 BlockSize = 512;
	static_assert(FMath::IsPowerOfTwo(BlockSize));

	FShaderJobData Data[BlockSize];
};

class FShaderJobDataMap
{
public:
	FShaderJobDataMap()
	{
		// Reserve so we don't need a special case for an empty HashTable array
		Reserve(FShaderJobDataBlock::BlockSize);
	}

	FShaderJobData* Find(const FShaderCompilerInputHash& Key);
	FShaderJobCacheRef FindOrAdd(const FShaderCompilerInputHash& Key);

	FORCEINLINE int32 Num() const
	{
		return NumItems;
	}

	FORCEINLINE FShaderJobData& operator[](int32 Index)
	{
		check((uint32)Index < (uint32)NumItems);
		return DataBlocks[Index / FShaderJobDataBlock::BlockSize].Data[Index & (FShaderJobDataBlock::BlockSize - 1)];
	}

	FORCEINLINE const FShaderJobData& operator[](int32 Index) const
	{
		return (*const_cast<FShaderJobDataMap*>(this))[Index];
	}

	uint64 GetAllocatedSize() const
	{
		return DataBlocks.GetAllocatedSize() + HashTable.GetAllocatedSize();
	}

	void RemoveLeadingBlocks(int32 BlocksToRemove)
	{
		check(BlocksToRemove <= DataBlocks.Num() && BlocksToRemove > 0);
		DataBlocks.RemoveAt(0, BlocksToRemove);
		NumItems -= BlocksToRemove * FShaderJobDataBlock::BlockSize;
		check(NumItems >= 0);

		if (NumItems == 0)
		{
			// If we happened to remove ALL the items, reserve again, as done in the constructor
			Reserve(FShaderJobDataBlock::BlockSize);
		}
		else
		{
			// Otherwise, we need to rehash, as all item indices will have changed
			ReHash(GetDesiredHashTableSize());
		}
	}

private:
	void ReHash(int32 HashTableSize);
	void Reserve(int32 NumReserve);

	int32 GetDesiredHashTableSize() const
	{
		return FMath::RoundUpToPowerOfTwo(DataBlocks.Num() * FShaderJobDataBlock::BlockSize * 2);
	}

	/** An indirect array of blocks is used, so data elements never move in memory when the table grows */
	TIndirectArray<FShaderJobDataBlock> DataBlocks;
	int32 NumItems = 0;

	/** Power of two hash table with linear probing */
	TArray<uint32> HashTable;
	uint32 HashTableMask = 0;
};


struct FShaderJobCacheStoredOutput
{
private:
	/** How many times this output is referenced by the cached jobs */
	int32 NumReferences = 0;

public:

	/** How many times this output has been returned as a cached result, no matter the input hash */
	int32 NumHits = 0;

	/** Canned output */
	FSharedBuffer JobOutput;

	/** Separate blobs for shader code */
	TArray<FCompositeBuffer> JobCode;

	/** Separate blobs for shader symbols */
	TArray<FCompressedBuffer> JobSymbols;

	/** Path to where the cached debug info is stored. */
	FString CachedDebugInfoPath;

	/** Similar to FRefCountedObject AddRef, but not atomic */
	int32 AddRef()
	{
		++NumReferences;

		return NumReferences;
	}

	int32 GetNumReferences() const
	{
		return NumReferences;
	}

	/** Similar to FRefCountedObject Release, but not atomic */
	int32 Release()
	{
		checkf(NumReferences >= 0, TEXT("Attempting to release shader job cache output that was already released"));

		--NumReferences;

		const int32 RemainingNumReferences = NumReferences;

		if (RemainingNumReferences == 0)
		{
			delete this;
		}

		return RemainingNumReferences;
	}

	uint64 GetAllocatedSize() const
	{
		uint64 Size = JobOutput.GetSize() + sizeof(*this);
		for (const FCompositeBuffer& CodeBuf : JobCode)
		{
			Size += CodeBuf.GetSize();
		}
		for (const FCompressedBuffer& SymbolBuf : JobSymbols)
		{
			Size += SymbolBuf.GetCompressedSize();
		}
		return Size;
	}
};


/**
 * Class that provides a lock striped hash table of jobs, to reduce lock contention when adding or removing jobs
 */
class FShaderCompilerJobTable
{
public:
	static constexpr int32 NUM_STRIPE_BITS = 6;
	static constexpr int32 NUM_STRIPES = 1 << NUM_STRIPE_BITS;

	/** We want to use the high bits of the hash for the stripe index, as it won't have influence on the hash table index within the stripe */
	static constexpr int32 STRIPE_SHIFT = 32 - NUM_STRIPE_BITS;

	template<typename JobType, typename KeyType>
	FShaderCommonCompileJobPtr PrepareJob(uint32 InId, const KeyType& InKey, EShaderCompileJobPriority InPriority, bool& bOutNewJob)
	{
		const uint32 Hash = InKey.MakeHash(InId);
		FLockStripeData& Stripe = GetStripe(JobType::Type, Hash);

		FWriteScopeLock Locker(Stripe.StripeLock);

		JobType* ResultJob = InternalFindJob<JobType>(Hash, InId, InKey);
		bOutNewJob = false;

		if (ResultJob == nullptr)
		{
			ResultJob = new JobType(Hash, InId, InPriority, InKey);
			InternalAddJob(ResultJob);
			bOutNewJob = true;
		}

		return ResultJob;
	}

	// PrepareJob creates a job with the given key if it's unique, while this adds an existing job, typically one that is cloned from another job
	void AddExistingJob(FShaderCommonCompileJob* InJob)
	{
		FLockStripeData& Stripe = GetStripe(InJob->Type, InJob->Hash);

		FWriteScopeLock Locker(Stripe.StripeLock);
		InternalAddJob(InJob);
	}

	void RemoveJob(FShaderCommonCompileJob* InJob)
	{
		FLockStripeData& Stripe = GetStripe(InJob->Type, InJob->Hash);

		FWriteScopeLock Locker(Stripe.StripeLock);

		const int32 JobIndex = InJob->JobIndex;

		check(JobIndex != INDEX_NONE);
		check(Stripe.Jobs[JobIndex] == InJob);
		check(InJob->PendingPriority == EShaderCompileJobPriority::None);
		InJob->JobIndex = INDEX_NONE;

		Stripe.JobHash.Remove(InJob->Hash, JobIndex);
		Stripe.FreeIndices.Add(JobIndex);
		Stripe.Jobs[JobIndex].SafeRelease();
	}

private:
	template<typename JobType, typename KeyType>
	JobType* InternalFindJob(uint32 InJobHash, uint32 InJobId, const KeyType& InKey) const
	{
		const FLockStripeData& Stripe = GetStripe(JobType::Type, InJobHash);

		uint32 CurrentPriorityIndex = 0u;
		int32 CurrentIndex = INDEX_NONE;
		for (int32 Index = Stripe.JobHash.First(InJobHash); Stripe.JobHash.IsValid(Index); Index = Stripe.JobHash.Next(Index))
		{
			const FShaderCommonCompileJob* Job = Stripe.Jobs[Index].GetReference();
			check(Job->Type == JobType::Type);

			// We find the job that matches the key with the highest priority
			if (Job->Id == InJobId &&
				(uint32)Job->Priority >= CurrentPriorityIndex &&
				static_cast<const JobType*>(Job)->Key == InKey)
			{
				CurrentPriorityIndex = (uint32)Job->Priority;
				CurrentIndex = Index;
			}
		}

		return CurrentIndex != INDEX_NONE ? static_cast<JobType*>(Stripe.Jobs[CurrentIndex].GetReference()) : nullptr;
	}

	void InternalAddJob(FShaderCommonCompileJob* InJob)
	{
		FLockStripeData& Stripe = GetStripe(InJob->Type, InJob->Hash);

		int32 JobIndex = INDEX_NONE;
		if (Stripe.FreeIndices.Num() > 0)
		{
			JobIndex = Stripe.FreeIndices.Pop(EAllowShrinking::No);
			check(!Stripe.Jobs[JobIndex].IsValid());
			Stripe.Jobs[JobIndex] = InJob;
		}
		else
		{
			JobIndex = Stripe.Jobs.Add(InJob);
		}

		check(Stripe.Jobs[JobIndex].IsValid());
		Stripe.JobHash.Add(InJob->Hash, JobIndex);

		check(InJob->Priority != EShaderCompileJobPriority::None);
		check(InJob->PendingPriority == EShaderCompileJobPriority::None);
		check(InJob->JobIndex == INDEX_NONE);
		InJob->JobIndex = JobIndex;
	}

	struct FLockStripeData
	{
		TArray<FShaderCommonCompileJobPtr> Jobs;
		TArray<int32> FreeIndices;
		FHashTable JobHash;
		FRWLock StripeLock;
	};

	FLockStripeData Stripes[NumShaderCompileJobTypes][NUM_STRIPES];

	FORCEINLINE FLockStripeData& GetStripe(EShaderCompileJobType JobType, uint32 Hash)
	{
		checkf((uint8)JobType < (uint8)NumShaderCompileJobTypes, TEXT("Out of range JobType index %u"), (uint8)JobType);
		return Stripes[(uint8)JobType][Hash >> STRIPE_SHIFT];
	}
	FORCEINLINE const FLockStripeData& GetStripe(EShaderCompileJobType JobType, uint32 Hash) const
	{
		checkf((uint8)JobType < (uint8)NumShaderCompileJobTypes, TEXT("Out of range JobType index %u"), (uint8)JobType);
		return Stripes[(uint8)JobType][Hash >> STRIPE_SHIFT];
	}
};

/** Private implementation class for FShaderCompileJobCollection */
class FShaderJobCache
{
public:
	FShaderJobCache(FCriticalSection& InCompileQueueSection);
	~FShaderJobCache();

	// Returns job pointer for new job, otherwise returns NULL
	template <typename JobType, typename KeyType>
	JobType* PrepareJob(uint32 InId, const KeyType& InKey, EShaderCompileJobPriority InPriority)
	{
		bool bNewJob;
		FShaderCommonCompileJobPtr Result = JobTable.PrepareJob<JobType>(InId, InKey, InPriority, bNewJob);

		if (bNewJob)
		{
			// If it's a new job, return it -- it's OK to cast the ref-counted pointer to a raw pointer, because JobTable
			// itself has a reference to the job, and a newly added job hasn't been submitted yet, so it can't make a
			// round trip through the pipeline and be released until that happens.
			return (JobType*)Result.GetReference();
		}
		else if (InPriority > Result->Priority)
		{
			// Or if the priority changed, update that
			InternalSetPriority(Result, InPriority);
		}

		return nullptr;
	}

	void RemoveJob(FShaderCommonCompileJob* InJob)
	{
		JobTable.RemoveJob(InJob);
	}

	int32 RemoveAllPendingJobsWithId(uint32 InId);

	void SubmitJob(FShaderCommonCompileJob* Job);
	void SubmitJobs(const TArray<FShaderCommonCompileJobPtr>& InJobs);
	void TransferJobsToLocalWorker(const TArray<FShaderCommonCompileJobPtr>& InJobs);
	void OnSubmitJobComplete();

	/** This is an entry point for all jobs that have finished the compilation (whether real or cached). Can be called from multiple threads. */
	void ProcessFinishedJob(FShaderCommonCompileJob* FinishedJob, EShaderCompileJobStatus Status, EShaderCompileJobCompletionMethod Method = EShaderCompileJobCompletionMethod::Unset);

	/** Adds the job to cache. */
	void AddToCacheAndProcessPending(FShaderCommonCompileJob* FinishedJob);

	/** Populates caching stats in the given compiler stats struct. */
	void GetStats(FShaderCompilerStats& OutStats) const;

	int32 GetNumPendingJobs(EShaderCompileJobPriority InPriority) const;

	int32 GetNumOutstandingJobs() const;

	int32 GetNumPendingJobs() const;

	int32 GetPendingJobs(EShaderCompilerWorkerType InWorkerType, EShaderCompileJobPriority InPriority, int32 MinNumJobs, int32 MaxNumJobs, TArray<FShaderCommonCompileJobPtr>& OutJobs);

private:
	using FJobOutputHash = FBlake3Hash;
	using FStoredOutput = FShaderJobCacheStoredOutput;

	// cannot allow managing this from outside as the caching logic is not exposed
	inline int32 InternalSubtractNumOutstandingJobs(int32 Value)
	{
		const int32 PrevNumOutstandingJobs = NumOutstandingJobs.Subtract(Value);
		check(PrevNumOutstandingJobs >= Value);
		return PrevNumOutstandingJobs - Value;
	}

	FJobOutputHash ComputeJobHash(const FShaderCacheSerializeContext& SerializeContext)
	{
		FBlake3 Hasher;
		check(SerializeContext.HasData());
		Hasher.Update(SerializeContext.ShaderObjectData.GetData(), SerializeContext.ShaderObjectData.GetSize());
		for (const FCompositeBuffer& CodeBuf : SerializeContext.ShaderCode)
		{
			for (const FSharedBuffer& CodeBufSegment : CodeBuf.GetSegments())
			{
				Hasher.Update(CodeBufSegment.GetData(), CodeBufSegment.GetSize());
			}
		}
		return Hasher.Finalize();
	}

	void InternalSetPriority(FShaderCommonCompileJob* Job, EShaderCompileJobPriority InPriority);

	/** Looks for or adds an entry for the given hash in the cache.  Returns cached output if it exists, or may initialize DDC request if one has been issued. */
	FShaderCacheLoadContext FindOrAdd(const FShaderCompilerInputHash& Hash, FShaderCommonCompileJob* Job, const bool bCheckDDC);

	/** Find an existing item in the cache. */
	FShaderJobData* Find(const FShaderCompilerInputHash& Hash);

	/** Add a reference to a duplicate job (to the DuplicateJobs array) */
	void AddDuplicateJob(FShaderCommonCompileJob* DuplicateJob);

	/** Remove a reference to a duplicate job (from the DuplicateJobs array)  */
	void RemoveDuplicateJob(FShaderCommonCompileJob* DuplicateJob);

	/** Adds a job output to the cache */
	void AddJobOutput(FShaderJobData& JobData, const FShaderCommonCompileJob* FinishedJob, const FShaderCompilerInputHash& Hash, FShaderCacheSaveContext& SaveContext, int32 InitialHitCount, const bool bAddToDDC);

	/** Returns memory used by the cache*/
	uint64 GetAllocatedMemory() const;

	/** Compute memory used by the cache from scratch.  Should match GetAllocatedMemory() if CurrentlyAllocateMemory is being properly updated (useful for validation). */
	uint64 ComputeAllocatedMemory() const;

	/** Calculates current memory budget, in bytes */
	uint64 GetCurrentMemoryBudget() const;

	/** Cleans up oldest outputs to fit in the given memory budget */
	void CullOutputsToMemoryBudget(uint64 TargetBudgetBytes);

	/** Copied from TLinkedListBase::Unlink */
	FORCEINLINE static void Unlink(FShaderCommonCompileJob& Job)
	{
		if (Job.NextLink)
		{
			Job.NextLink->PrevLink = Job.PrevLink;
		}
		if (Job.PrevLink)
		{
			*Job.PrevLink = Job.NextLink;
		}
		// Make it safe to call Unlink again.
		Job.NextLink = nullptr;
		Job.PrevLink = nullptr;
	}

	/**
	 * Similar to TLinkedListBase::Unlink, but updates a Tail pointer if the Tail is unlinked.  The tail must
	 * originally be initialized as Tail = &Head.
	 */
	FORCEINLINE void UnlinkWithTail(FShaderCommonCompileJob& Job, FShaderCommonCompileJob**& Tail)
	{
		// Update tail if we are removing that element
		if (Tail == &Job.NextLink)
		{
			Tail = Job.PrevLink;
		}
		Unlink(Job);
	}

	/** Copied from TLinkedListBase::LinkHead */
	FORCEINLINE void LinkHead(FShaderCommonCompileJob& Job, FShaderCommonCompileJob*& Head)
	{
		if (Head != NULL)
		{
			Head->PrevLink = &Job.NextLink;
		}

		Job.NextLink = Head;
		Job.PrevLink = &Head;
		Head = &Job;
	}

	/** Copied from TLinkedListBase::LinkAfter */
	FORCEINLINE void LinkAfter(FShaderCommonCompileJob& Job, FShaderCommonCompileJob* After)
	{
		checkSlow(After != NULL);
		check(Job.NextLink == nullptr && Job.PrevLink == nullptr);

		Job.PrevLink = &After->NextLink;
		Job.NextLink = *Job.PrevLink;
		*Job.PrevLink = (FShaderCommonCompileJob*)&Job;

		if (Job.NextLink != NULL)
		{
			Job.NextLink->PrevLink = &Job.NextLink;
		}
	}

	/**
	 * Similar to TLinkedListBase::LinkHead, but uses atomic operations to allow multiple producer threads to add to the linked list
	 * without needing synchronization ("wait free").  Note that synchronization is required for other operations on the list, such
	 * as traversal or removal, as the list isn't in a fully valid state mid operation (Head always points to the latest item
	 * inserted, but it may not yet be linked with the rest of the items).  Synchronization is accomplished by using a read lock
	 * (which multiple threads can hold) for atomic insertion operations, and a write lock for all other operations.
	 */
	FORCEINLINE void LinkHeadAtomic(FShaderCommonCompileJob& Job, FShaderCommonCompileJob*& Head)
	{
		check(Job.NextLink == nullptr && Job.PrevLink == nullptr);

		// It's important that PrevLink is set before the InterlockedExchange, as a subsequent Head pointer exchange could write
		// another item and need to update PrevLink for this item before this function completes.
		Job.PrevLink = &Head;

		FShaderCommonCompileJob* OldHead = (FShaderCommonCompileJob*)FPlatformAtomics::InterlockedExchange((PTRINT*)&Head, (PTRINT)&Job);
		if (OldHead != nullptr)
		{
			OldHead->PrevLink = &Job.NextLink;
		}
		Job.NextLink = OldHead;
	}

	/**
	 * Variation that links a job at the tail of the list.  The tail must originally be initialized as Tail = &Head.  Similar to
	 * LinkHeadAtomic above (see more detailed comments there), a read lock is required for this operation.
	 */
	FORCEINLINE static void LinkTailAtomic(FShaderCommonCompileJob& Job, FShaderCommonCompileJob**& Tail)
	{
		check(Job.NextLink == nullptr && Job.PrevLink == nullptr);

		FShaderCommonCompileJob** OldTail = (FShaderCommonCompileJob**)FPlatformAtomics::InterlockedExchange((PTRINT*)&Tail, (PTRINT)&Job.NextLink);
		Job.PrevLink = OldTail;

		// Update previous tail's next pointer (or OldTail may be pointing at Head if list was empty)
		*OldTail = (FShaderCommonCompileJob*)&Job;
	}

	/** Links job into linked list with its given Priority */
	FORCEINLINE void LinkJobWithPriority(FShaderCommonCompileJob& Job)
	{
		int32 PriorityIndex = (int32)Job.Priority;
		check((uint32)PriorityIndex < (uint32)NumShaderCompileJobPriorities);
		check(Job.PendingPriority == EShaderCompileJobPriority::None);
		NumPendingJobs[PriorityIndex]++;
#if UE_SHADERCOMPILER_FIFO_JOB_EXECUTION
		LinkTailAtomic(Job, PendingJobsTail[PriorityIndex]);
#else
		LinkHeadAtomic(Job, PendingJobsHead[PriorityIndex]);
#endif
		Job.PendingPriority = Job.Priority;
	}

	/** Unlinks job from linked list with its current PendingPriority */
	FORCEINLINE void UnlinkJobWithPriority(FShaderCommonCompileJob& Job)
	{
		int32 PriorityIndex = (int32)Job.PendingPriority;
		check((uint32)PriorityIndex < (uint32)NumShaderCompileJobPriorities);
		check(NumPendingJobs[PriorityIndex] > 0);
		NumPendingJobs[PriorityIndex]--;
#if UE_SHADERCOMPILER_FIFO_JOB_EXECUTION
		UnlinkWithTail(Job, PendingJobsTail[PriorityIndex]);
#else
		Unlink(Job);
#endif
		Job.PendingPriority = EShaderCompileJobPriority::None;
	}

	/** From FShaderCompilingManager, guards access to FShaderMapCompileResults written in ProcessFinishedJob */
	FCriticalSection& CompileQueueSection;

	/** Guards access to the structure */
	mutable FRWLock JobLock;

	/** Needed to detect if DDC query callback is completing in the same thread as SubmitJob */
	static thread_local bool bInSubmitJobThread;

	/** List of jobs waiting on SubmitJob task or DDC query (not yet added to a pending queue). */
	FShaderCommonCompileJob* PendingSubmitJobTaskJobs = nullptr;

	/** Queue of tasks that haven't been assigned to a worker yet. */
	TStaticArray<FShaderCommonCompileJob*, NumShaderCompileJobPriorities> PendingJobsHead;
	TStaticArray<std::atomic_int32_t, NumShaderCompileJobPriorities> NumPendingJobs;
#if UE_SHADERCOMPILER_FIFO_JOB_EXECUTION
	TStaticArray<FShaderCommonCompileJob**, NumShaderCompileJobPriorities> PendingJobsTail;
#endif

	/** Number of jobs currently being compiled.  This includes PendingJobs and any jobs that have been assigned to workers but aren't complete yet. */
	FThreadSafeCounter NumOutstandingJobs;

	/** Active jobs */
	FShaderCompilerJobTable JobTable;

	/* A lot of outputs can be duplicated, so they are deduplicated before storing */
	TMap<FJobOutputHash, FStoredOutput*> Outputs;

	TMap<FJobOutputHash, FString> CachedJobNames;

	/** Map of input hashes to job data (in flight jobs and output) */
	FShaderJobDataMap InputHashToJobData;

	/** List of duplicate jobs */
	TArray<FShaderCommonCompileJob*> DuplicateJobs;

	std::atomic<uint32> NumPendingSubmitJobs{ 0 };
	FEventRef AllSubmitJobsCompleteEvent;

	/** Statistics - total number of times we tried to Find() some input hash */
	uint64 TotalSearchAttempts = 0;

	/** Statistics - total number of times we succeded in Find()ing output for some input hash */
	uint64 TotalCacheHits = 0;

	/** Statistics - total number of times a duplicate job was added (duplicate jobs are processed when the original finishes compiling) */
	uint64 TotalCacheDuplicates = 0;

	/** Statistics - total number of times a per-shader DDC query was issued */
	uint64 TotalCacheDDCQueries = 0;

	/** Statistics - total number of times a per-shader DDC query succeeded for some input hash */
	uint64 TotalCacheDDCHits = 0;

	/** Statistics - allocated memory. If the number is non-zero, we can trust it as accurate. Otherwise, recalculate. */
	uint64 CurrentlyAllocatedMemory = 0;
};

thread_local bool FShaderJobCache::bInSubmitJobThread;

static FShaderJobData& GetShaderJobData(const FShaderJobCacheRef& CacheRef)
{
	check(CacheRef.Block);
	return CacheRef.Block->Data[CacheRef.IndexInBlock];
}

FShaderJobData* FShaderJobDataMap::Find(const FShaderCompilerInputHash& Key)
{
	// Search for key with linear probing
	for (uint32 TableIndex = GetTypeHash(Key) & HashTableMask; HashTable[TableIndex] != INDEX_NONE; TableIndex = (TableIndex + 1) & HashTableMask)
	{
		if ((*this)[TableIndex].InputHash == Key)
		{
			return &(*this)[TableIndex];
		}
	}
	return nullptr;
}

FShaderJobCacheRef FShaderJobDataMap::FindOrAdd(const FShaderCompilerInputHash& Key)
{
	// Search for key with linear probing
	uint32 TableIndex;
	for (TableIndex = GetTypeHash(Key) & HashTableMask; HashTable[TableIndex] != INDEX_NONE; TableIndex = (TableIndex + 1) & HashTableMask)
	{
		int32 ItemIndex = HashTable[TableIndex];
		if ((*this)[ItemIndex].InputHash == Key)
		{
			return FShaderJobCacheRef({ &DataBlocks[ItemIndex / FShaderJobDataBlock::BlockSize], ItemIndex & (FShaderJobDataBlock::BlockSize - 1), INDEX_NONE });
		}
	}

	// Ensure there is space for item
	Reserve(NumItems + 1);

	// Initialize allocated item
	int32 AllocatedIndex = NumItems++;
	FShaderJobCacheRef AllocatedItem({ &DataBlocks[AllocatedIndex / FShaderJobDataBlock::BlockSize], AllocatedIndex & (FShaderJobDataBlock::BlockSize - 1), INDEX_NONE });
	GetShaderJobData(AllocatedItem).InputHash = Key;

	// Add to empty spot in hash table
	HashTable[TableIndex] = AllocatedIndex;

	return AllocatedItem;
}

void FShaderJobDataMap::ReHash(int32 HashTableSize)
{
	// Resize table and rehash
	HashTable.SetNumUninitialized(HashTableSize);
	memset(HashTable.GetData(), 0xff, HashTable.Num() * HashTable.GetTypeSize());
	HashTableMask = HashTableSize - 1;

	for (int32 OuterIndex = 0; OuterIndex < DataBlocks.Num(); OuterIndex++)
	{
		FShaderJobData* Data = DataBlocks[OuterIndex].Data;

		for (int32 InnerIndex = 0; InnerIndex < FShaderJobDataBlock::BlockSize; InnerIndex++)
		{
			int32 Index = OuterIndex * FShaderJobDataBlock::BlockSize + InnerIndex;
			if (Index >= NumItems)
			{
				OuterIndex = DataBlocks.Num();
				break;
			}

			// Find table entry for key -- keys will be unique when rehashing, so we don't need to check for existing keys
			for (uint32 TableIndex = GetTypeHash(Data[InnerIndex].InputHash) & HashTableMask;; TableIndex = (TableIndex + 1) & HashTableMask)
			{
				if (HashTable[TableIndex] == INDEX_NONE)
				{
					HashTable[TableIndex] = Index;
					break;
				}
			}
		}
	}
}

void FShaderJobDataMap::Reserve(int32 NumReserve)
{
	if (NumReserve > DataBlocks.Num() * FShaderJobDataBlock::BlockSize)
	{
		while (NumReserve > DataBlocks.Num() * FShaderJobDataBlock::BlockSize)
		{
			DataBlocks.Add(new FShaderJobDataBlock);
		}

		int32 HashTableSize = GetDesiredHashTableSize();
		if (HashTableSize != HashTable.Num())
		{
			ReHash(HashTableSize);
		}
	}
}

void FShaderJobCache::CullOutputsToMemoryBudget(uint64 TargetBudgetBytes)
{
	// Track consecutive empty items.  We can delete empty blocks from the front of the map at the end.
	int32 ConsecutiveEmptyItems = 0;
	uint64 EmptyBlockSavings = 0;

	// We don't cull items from the last block

	for (int32 ItemIndex = 0; ItemIndex < InputHashToJobData.Num(); ItemIndex++)
	{
		FShaderJobData& JobData = InputHashToJobData[ItemIndex];

		// Check if we are in budget yet
		if (CurrentlyAllocatedMemory - EmptyBlockSavings <= TargetBudgetBytes)
		{
			break;
		}

		// We can only free this output if there is no in-flight job
		if (JobData.JobInFlight == nullptr)
		{
			// Empty this item out (if not already empty), by removing the reference to the output and zeroing it out
			if (!JobData.OutputHash.IsZero())
			{
				FStoredOutput** FoundStoredOutput = Outputs.Find(JobData.OutputHash);

				if (FoundStoredOutput)
				{
					FStoredOutput* StoredOutput = *FoundStoredOutput;
					checkf(StoredOutput, TEXT("Invalid entry found in FShaderJobCache Output hash table. All values are expected to be valid pointers."));

					const uint64 OutputSize = StoredOutput->GetAllocatedSize();

					// Decrement reference count and remove cached object if it's no longer referenced by any input hashes
					if (StoredOutput->Release() == 0)
					{
						Outputs.Remove(JobData.OutputHash);
						CachedJobNames.Remove(JobData.OutputHash);
						CurrentlyAllocatedMemory -= OutputSize;
					}
				}

				JobData.OutputHash.Reset();
			}

			// Track if this is another consecutive empty item
			if (ItemIndex == ConsecutiveEmptyItems)
			{
				ConsecutiveEmptyItems++;

				// Take into account that we will be removing empty job data blocks at the end, by adding the savings when we reach a full block
				if ((ConsecutiveEmptyItems & (FShaderJobDataBlock::BlockSize - 1)) == 0)
				{
					EmptyBlockSavings += sizeof(FShaderJobDataBlock);
				}
			}
		}
	}

	int32 ConsecutiveEmptyBlocks = ConsecutiveEmptyItems / FShaderJobDataBlock::BlockSize;
	if (ConsecutiveEmptyBlocks > 0)
	{
		uint64 InputHashToJobDataOriginalSize = InputHashToJobData.GetAllocatedSize();

		InputHashToJobData.RemoveLeadingBlocks(ConsecutiveEmptyBlocks);

		CurrentlyAllocatedMemory += InputHashToJobData.GetAllocatedSize() - InputHashToJobDataOriginalSize;
	}
}


FShaderCompileJobCollection::FShaderCompileJobCollection(FCriticalSection& InCompileQueueSection)
{
	PrintStatsCmd = IConsoleManager::Get().RegisterConsoleCommand(
		TEXT("r.ShaderCompiler.PrintStats"),
		TEXT("Prints out to the log the stats for the shader compiler."),
		FConsoleCommandDelegate::CreateRaw(this, &FShaderCompileJobCollection::HandlePrintStats),
		ECVF_Default
	);

	JobsCache = MakePimpl<FShaderJobCache>(InCompileQueueSection);
}

// Pass through functions to inner FShaderJobCache implementation class
FShaderCompileJob* FShaderCompileJobCollection::PrepareJob(uint32 InId, const FShaderCompileJobKey& InKey, EShaderCompileJobPriority InPriority)
{
	return JobsCache->PrepareJob<FShaderCompileJob>(InId, InKey, InPriority);
}

FShaderPipelineCompileJob* FShaderCompileJobCollection::PrepareJob(uint32 InId, const FShaderPipelineCompileJobKey& InKey, EShaderCompileJobPriority InPriority)
{
	return JobsCache->PrepareJob<FShaderPipelineCompileJob>(InId, InKey, InPriority);
}

void FShaderCompileJobCollection::RemoveJob(FShaderCommonCompileJob* InJob)
{
	JobsCache->RemoveJob(InJob);
}

int32 FShaderCompileJobCollection::RemoveAllPendingJobsWithId(uint32 InId)
{
	return JobsCache->RemoveAllPendingJobsWithId(InId);
}

void FShaderCompileJobCollection::SubmitJobs(const TArray<FShaderCommonCompileJobPtr>& InJobs)
{
	JobsCache->SubmitJobs(InJobs);
}

void FShaderCompileJobCollection::TransferJobsToLocalWorker(const TArray<FShaderCommonCompileJobPtr>& InJobs)
{
	JobsCache->TransferJobsToLocalWorker(InJobs);
}

void FShaderCompileJobCollection::ProcessFinishedJob(FShaderCommonCompileJob* FinishedJob, EShaderCompileJobStatus Status)
{
	JobsCache->ProcessFinishedJob(FinishedJob, Status);
}

void FShaderCompileJobCollection::AddToCacheAndProcessPending(FShaderCommonCompileJob* FinishedJob)
{
	JobsCache->AddToCacheAndProcessPending(FinishedJob);
}

void FShaderCompileJobCollection::GetCachingStats(FShaderCompilerStats& OutStats) const
{
	JobsCache->GetStats(OutStats);
}

int32 FShaderCompileJobCollection::GetNumPendingJobs(EShaderCompileJobPriority InPriority) const
{
	return JobsCache->GetNumPendingJobs(InPriority);
}

int32 FShaderCompileJobCollection::GetNumOutstandingJobs() const
{
	return JobsCache->GetNumOutstandingJobs();
}

int32 FShaderCompileJobCollection::GetNumPendingJobs() const
{
	return JobsCache->GetNumPendingJobs();
}

int32 FShaderCompileJobCollection::GetPendingJobs(EShaderCompilerWorkerType InWorkerType, EShaderCompileJobPriority InPriority, int32 MinNumJobs, int32 MaxNumJobs, TArray<FShaderCommonCompileJobPtr>& OutJobs)
{
	return JobsCache->GetPendingJobs(InWorkerType, InPriority, MinNumJobs, MaxNumJobs, OutJobs);
}

void FShaderCompileJobCollection::HandlePrintStats()
{
	GShaderCompilingManager->PrintStats();
}

void FShaderJobCache::InternalSetPriority(FShaderCommonCompileJob* Job, EShaderCompileJobPriority InPriority)
{
	if (Job->PendingPriority != EShaderCompileJobPriority::None)
	{
		// Need write lock to call UnlinkJobWithPriority
		FWriteScopeLock Locker(JobLock);

		// Check priority again, as the job may have been kicked off by another thread while waiting on the lock
		if (Job->PendingPriority != EShaderCompileJobPriority::None)
		{
			// Job hasn't started yet, move it to the pending list for the new priority
			check(Job->PendingPriority == Job->Priority);
			UnlinkJobWithPriority(*Job);

			ensure(Job->bInputHashSet);
			Job->Priority = InPriority;
			LinkJobWithPriority(*Job);
		}
	}
}

int32 FShaderJobCache::RemoveAllPendingJobsWithId(uint32 InId)
{
	int32 NumRemoved = 0;

#if WITH_EDITOR
	TArray<FShaderCommonCompileJobPtr> JobsWithRequestsToCancel;
#endif
	{
		// Look for jobs that are waiting on a SubmitJob task or async DDC query.  These can just be unlinked which will cause them to be
		// discarded in SubmitJob or the DDC completion callback.  We also need to get a list of jobs with DDC requests to cancel.  We
		// can't cancel the requests inside the loop, as the response callback uses JobLock, and it will deadlock.  We also need a
		// reference pointer to the jobs, so the jobs (and the TPimplPtr<UE::DerivedData::FRequestOwner> contained therein) can't be
		// deleted while a DDC completion callback is in flight, which also leads to a deadlock.
		FWriteScopeLock Locker(JobLock);
		for (FShaderCommonCompileJobIterator It(PendingSubmitJobTaskJobs); It;)
		{
			FShaderCommonCompileJob& Job = *It;
			It.Next();

			if (Job.Id == InId)
			{
				Unlink(Job);		// from PendingSubmitJobTaskJobs
				RemoveJob(&Job);
				++NumRemoved;

#if WITH_EDITOR
				if (Job.RequestOwner)
				{
					JobsWithRequestsToCancel.Add(&Job);
				}
#endif
			}
		}
	}

#if WITH_EDITOR
	for (FShaderCommonCompileJobPtr JobWithRequestToCancel : JobsWithRequestsToCancel)
	{
		// Cancelling should short circuit the request, and make "Wait" finish immediately
		JobWithRequestToCancel->RequestOwner->Cancel();
		JobWithRequestToCancel->RequestOwner->Wait();
	}
#endif

	{
		FWriteScopeLock Locker(JobLock);
		for (int32 PriorityIndex = 0; PriorityIndex < NumShaderCompileJobPriorities; ++PriorityIndex)
		{
			for (FShaderCommonCompileJobIterator It(PendingJobsHead[PriorityIndex]); It;)
			{
				FShaderCommonCompileJob& Job = *It;
				It.Next();

				if (Job.Id == InId)
				{
					if (Job.JobCacheRef.Block)
					{
						FShaderJobData& JobData = GetShaderJobData(Job.JobCacheRef);

						check(JobData.JobInFlight == &Job);

						// If we are removing an in-flight job, we need to promote a duplicate to be the new in-flight job, if present.
						// Make sure the duplicate we choose doesn't have the same ID as what we're removing.
						FShaderCommonCompileJob* DuplicateJob;
						for (DuplicateJob = JobData.DuplicateJobsWaitList; DuplicateJob; DuplicateJob = DuplicateJob->NextLink)
						{
							if (DuplicateJob->Id != InId)
							{
								break;
							}
						}

						if (DuplicateJob)
						{
							// Advance head if we are unlinking the head, then remove
							if (JobData.DuplicateJobsWaitList == DuplicateJob)
							{
								JobData.DuplicateJobsWaitList = DuplicateJob->NextLink;
							}
							Unlink(*DuplicateJob);
							RemoveDuplicateJob(DuplicateJob);

							// Unset the execution method on the previously "duplicate" job; this will now be set by whatever executor picks up the job.
							DuplicateJob->JobStatusPtr->SetCompletionMethod(EShaderCompileJobCompletionMethod::Unset);

							// Add it as pending at the appropriate priority
							GShaderCompilerStats->RegisterNewPendingJob(*DuplicateJob);

							LinkJobWithPriority(*DuplicateJob);
						}

						// DuplicateJob will be nullptr if there was no duplicate to promote
						JobData.JobInFlight = DuplicateJob;
					}

					check((int32)Job.PendingPriority == PriorityIndex);
					UnlinkJobWithPriority(Job);
					RemoveJob(&Job);
					++NumRemoved;
				}
			}
		}

		// Also look into duplicate jobs that are cached -- we don't increment in the "for" loop because the current item may be deleted
		for (int32 DuplicateIndex = 0; DuplicateIndex < DuplicateJobs.Num();)
		{
			FShaderCommonCompileJob* DuplicateJob = DuplicateJobs[DuplicateIndex];
			check(DuplicateJob->JobCacheRef.DuplicateIndex == DuplicateIndex);

			if (DuplicateJob->Id == InId)
			{
				FShaderJobData& JobData = GetShaderJobData(DuplicateJob->JobCacheRef);

				// if we're removing the list head, we need to update it to the next
				if (JobData.DuplicateJobsWaitList == DuplicateJob)
				{
					JobData.DuplicateJobsWaitList = JobData.DuplicateJobsWaitList->NextLink;
				}

				// This removes the current job (at DuplicateIndex), so we don't increment in this case
				RemoveDuplicateJob(DuplicateJob);

				// Duplicate jobs are in their own list, not one of the priority lists, so don't use UnlinkJobWithPriority
				check(DuplicateJob->PendingPriority == EShaderCompileJobPriority::None);
				Unlink(*DuplicateJob);
				RemoveJob(DuplicateJob);
				++NumRemoved;
			}
			else
			{
				// Didn't remove a job, increment!
				DuplicateIndex++;
			}
		}
	}

	InternalSubtractNumOutstandingJobs(NumRemoved);

	return NumRemoved;
}

void FShaderJobCache::TransferJobsToLocalWorker(const TArray<FShaderCommonCompileJobPtr>& InJobs)
{
	FWriteScopeLock Locker(JobLock);

	for (const FShaderCommonCompileJobPtr& Job : InJobs)
	{
		// Mark job as begin transferred to avoid duplicate registration as an assigned job
		Job->bTransferred = true;

		// If job was already put into a priority bucket, remove it from the old priority
		if (Job->PendingPriority != EShaderCompileJobPriority::None)
		{
			check(Job->PendingPriority == Job->Priority);
			UnlinkJobWithPriority(*Job);
		}

		// Put job into new priority bucket
		ensure(Job->bInputHashSet);
		Job->Priority = EShaderCompileJobPriority::ForceLocal;
		LinkJobWithPriority(*Job);

		// Reset status and worker type
		Job->UpdateStatus(EShaderCompileJobStatus::Queued);
		Job->CurrentWorker = EShaderCompilerWorkerType::None;
	}
}

void FShaderJobCache::OnSubmitJobComplete()
{
	if (NumPendingSubmitJobs.fetch_sub(1, std::memory_order_acq_rel) == 1)
	{
		AllSubmitJobsCompleteEvent->Trigger();
	}
}

void FShaderJobCache::SubmitJob(FShaderCommonCompileJob* Job)
{
	// Set thread local so DDC query callback can detect if it's in the same thread, and we need to run through the non-async code path.
	struct InSubmitJobScope
	{
		InSubmitJobScope() { bInSubmitJobThread = true; }
		~InSubmitJobScope() { bInSubmitJobThread = false; }
	};
	InSubmitJobScope InSubmitJob;

	check(Job->Priority != EShaderCompileJobPriority::None);
	check(Job->PendingPriority == EShaderCompileJobPriority::None);

	const int32 PriorityIndex = (int32)Job->Priority;
	bool bNewJob = true;
	bool bJobCacheLocked = false;

	// check caches unless we're running in validation mode (which runs _all_ jobs and compares hashes of outputs)
	if (!ShaderCompiler::IsJobCacheDebugValidateEnabled())
	{
		bJobCacheLocked = true;

		const FShaderCompilerInputHash& InputHash = Job->GetInputHash();

		const bool bCheckDDC = GShaderCompilerPerShaderDDCGlobal || !(Job->bIsDefaultMaterial || Job->bIsGlobalShader);

		// We don't use a scope here, because we need to release this lock before calling ProcessFinishedJob, which needs to acquire
		// CompileQueueSection.  It's not safe to acquire CompileQueueSection where JobLock is locked first, as it will cause
		// deadlocks due to FShaderCompileThreadRunnable::CompilingLoop calling GetPendingJobs, which acquires those two locks in
		// the opposite order.
		double StallStart = FPlatformTime::Seconds();
		JobLock.WriteLock();

		// Job was linked in PendingSubmitJobTaskJobs before calling SubmitJob -- if it's not linked now, it means it was cancelled via
		// call to RemoveAllPendingJobsWithId, so we can ignore it and just return.
		if (!Job->PrevLink)
		{
			UE_LOGF(LogShaderCompilers, Display, "Cancelled job 0x%p with pending SubmitJob call.", Job);
			Job->UpdateStatus(EShaderCompileJobStatus::Cancelled);
			JobLock.WriteUnlock();
			return;
		}
		check(Job->JobIndex != INDEX_NONE);

		FShaderCacheLoadContext LoadContext = FindOrAdd(InputHash, Job, bCheckDDC);

		// see if there are already cached results for this job that were returned synchronously by FindOrAdd
		if (LoadContext.HasData() && !Job->bBypassCache)
		{
			Unlink(*Job);		// from PendingSubmitJobTaskJobs

			// Need to release the lock before calling ProcessFinishedJob, as mentioned above (and it's also good for performance to 
			// release the lock before the relatively costly "SerializeOutput" call).
			JobLock.WriteUnlock();
			bNewJob = false;
			bJobCacheLocked = false;

			UE_LOGF(LogShaderCompilers, UE_SHADERCACHE_LOG_LEVEL, "There is already a cached job with the ihash %ls, processing the new one immediately.", *LexToString(InputHash));

			Job->SerializeOutput(LoadContext);

			// finish the job instantly
			ProcessFinishedJob(Job, EShaderCompileJobStatus::Completed, EShaderCompileJobCompletionMethod::CacheHit);
		}
		else
		{
			FShaderJobData& JobData = GetShaderJobData(Job->JobCacheRef);
			// if there's an in flight job for this cache ref already another job with the same input hash is already being worked on
			bool bDuplicateJob = JobData.JobInFlight.IsValid(); 			
			// update the completion method for a duplicate job immediately (so the status reflects that it's a duplicate while it's in queued state)
			// keep it unset if it's a primary job that will be executed; the execution method will be set when it's picked up by a compilation thread
			Job->UpdateStatus(EShaderCompileJobStatus::Queued, bDuplicateJob ? EShaderCompileJobCompletionMethod::Duplicate : EShaderCompileJobCompletionMethod::Unset);
			if (bDuplicateJob)
			{
				UE_LOGF(LogShaderCompilers, UE_SHADERCACHE_LOG_LEVEL, "There is an outstanding job with the ihash %ls, not submitting another one (adding to wait list).", *LexToString(InputHash));

				Unlink(*Job);		// from PendingSubmitJobTaskJobs

				// because of the cloned jobs, we need to maintain a separate mapping
				FShaderCommonCompileJob** WaitListHead = &JobData.DuplicateJobsWaitList;
				if (*WaitListHead)
				{
					LinkAfter(*Job, *WaitListHead);
				}
				else
				{
					*WaitListHead = Job;
				}
				++TotalCacheDuplicates;
				AddDuplicateJob(Job);
				JobLock.WriteUnlock();
				bNewJob = false;
				bJobCacheLocked = false;
			}
			else
			{
				// track new jobs so we can dedupe them
				JobData.JobInFlight = Job;
			}
		}
	}
	else if (ShaderCompiler::IsJobCacheDebugValidateEnabled())
	{
		const FShaderCompilerInputHash& InputHash = Job->GetInputHash();
		const bool bCheckDDC = !(Job->bIsDefaultMaterial || Job->bIsGlobalShader);
		JobLock.WriteLock();
		FShaderCacheLoadContext LoadContext = FindOrAdd(InputHash, Job, bCheckDDC);
		bJobCacheLocked = true;
	}

	// new job
	if (bNewJob)
	{
		GShaderCompilerStats->RegisterNewPendingJob(*Job);
		ensure(Job->bInputHashSet);

		// If cache is disabled, we skipped the code that grabs the write lock above, so we need to do it here, before modifying the pending queue
		if (bJobCacheLocked == false)
		{
			bJobCacheLocked = true;
			JobLock.WriteLock();

			// Job was linked in PendingSubmitJobTaskJobs before calling SubmitJob -- if it's not linked now, it means it was cancelled via
			// call to RemoveAllPendingJobsWithId, so we can ignore it and just return.
			if (!Job->PrevLink)
			{
				UE_LOGF(LogShaderCompilers, Log, "Cancelled job 0x%p with pending SubmitJob call.", Job);
				Job->UpdateStatus(EShaderCompileJobStatus::Cancelled);
				JobLock.WriteUnlock();
				return;
			}
			check(Job->JobIndex != INDEX_NONE);
		}

		// If an async DDC request is in flight, that will add the job to the pending queue for processing when the request completes,
		// if the request didn't find a result.  Otherwise we add it to the pending queue immediately.
		if (Job->RequestOwner.IsValid() == false)
		{
			check(Job->PrevLink);
			Unlink(*Job);		// from PendingSubmitJobTaskJobs

			LinkJobWithPriority(*Job);
		}
	}

	if (bJobCacheLocked)
	{
		JobLock.WriteUnlock();
	}
}

void FShaderJobCache::SubmitJobs(const TArray<FShaderCommonCompileJobPtr>& InJobs)
{
	if (InJobs.Num() > 0)
	{
		// all jobs (not just actually submitted ones) count as outstanding. This needs to be done early because
		// we may fulfill some of the jobs from the cache (and we will be subtracting them)
		NumOutstandingJobs.Add(InJobs.Num());

		{
			// Add pending jobs to a list to support cancelling while SubmitJob tasks or async DDC queries are in flight
			FWriteScopeLock JobLocker(JobLock);
			for (FShaderCommonCompileJob* Job : InJobs)
			{
				LinkHead(*Job, PendingSubmitJobTaskJobs);
			}
		}

		NumPendingSubmitJobs.fetch_add(InJobs.Num(), std::memory_order_relaxed);
		for (const FShaderCommonCompileJobPtr& Job : InJobs)
		{
			UE::Tasks::ETaskPriority Prio = IsRunningCookCommandlet() ? UE::Tasks::ETaskPriority::Normal : UE::Tasks::ETaskPriority::BackgroundNormal;
			UE::Tasks::Launch(UE_SOURCE_LOCATION, [Job, this]()
				{
					TRACE_CPUPROFILER_EVENT_SCOPE(ShaderJobTask);

					if (GShaderCompilerDebugStallSubmitJob > 0)
					{
						FPlatformProcess::Sleep(GShaderCompilerDebugStallSubmitJob * 0.001f);
					}

					if (IsEngineExitRequested())
					{
						// we do not need to unlink the job here as normal shutdown flow of anything referencing pending shader jobs will call the cancellation path;
						// we just need to ensure that any tasks that still execute after this cancellation occurs decrement the atomic counter and possibly signal the
						// event that we wait on in the destructor (to ensure none of these tasks are outstanding before we start unloading modules)
						OnSubmitJobComplete();
						return;
					}

					const bool bSubmitJob = PreprocessShader(Job);
					Job->UpdateInputHash();
					Job->UpdateStatus(EShaderCompileJobStatus::Ready);

					if (bSubmitJob)
					{
						SubmitJob(Job);
					}
					else // if preprocessing ran and failed, finish the job immediately
					{
						JobLock.WriteLock();
						Unlink(*Job); // from PendingSubmitJobTaskJobs
						JobLock.WriteUnlock();
						ProcessFinishedJob(Job, EShaderCompileJobStatus::Skipped);
					}

					OnSubmitJobComplete();
				}, Prio);
		}
	}
}

void FShaderJobCache::ProcessFinishedJob(FShaderCommonCompileJob* FinishedJob, EShaderCompileJobStatus Status, EShaderCompileJobCompletionMethod CompletionMethod)
{
	FinishedJob->UpdateStatus(Status, CompletionMethod);

	FShaderDebugDataContext Ctx;
	FinishedJob->OnComplete(Ctx);

	if (!FinishedJob->JobStatusPtr->WasCompilationSkipped())
	{
		AddToCacheAndProcessPending(FinishedJob);
	}

	GShaderCompilerStats->RegisterFinishedJob(*FinishedJob);

	{
		// Need to protect writes to FShaderMapCompileResults
		FScopeLock Lock(&CompileQueueSection);
		FShaderMapCompileResults& ShaderMapResults = *(FinishedJob->PendingShaderMap);
		ShaderMapResults.FinishedJobs.Add(FinishedJob);
		ShaderMapResults.bAllJobsSucceeded = ShaderMapResults.bAllJobsSucceeded && FinishedJob->bSucceeded;

		const int32 NumPendingJobsForSM = ShaderMapResults.NumPendingJobs.Decrement();
		checkf(NumPendingJobsForSM >= 0, TEXT("Problem tracking pending jobs for a SM (%d), number of pending jobs (%d) is negative!"), FinishedJob->Id, NumPendingJobsForSM);
	}

	InternalSubtractNumOutstandingJobs(1);
}

void FShaderJobCache::AddToCacheAndProcessPending(FShaderCommonCompileJob* FinishedJob)
{
	// Cloned jobs won't include an entry in the job cache, so skip the caching logic.  The non-cloned version of the same
	// job will handle adding data to the cache when it completes.
	if (!FinishedJob->JobCacheRef.Block)
	{
		return;
	}

	ensureMsgf(FinishedJob->bInputHashSet, TEXT("Finished job didn't have input hash set, was shader compiler jobs cache toggled runtime?"));

	const FShaderCompilerInputHash& InputHash = FinishedJob->GetInputHash();
	FShaderCacheSaveContext SaveContext;
	FinishedJob->SerializeOutput(SaveContext);
	// Explicitly finalize the serialization to generate the job struct FSharedBuffer since it's needed below in the case
	// we need to process/populate any duplicate job results
	SaveContext.Finalize();

	FShaderJobData& JobData = GetShaderJobData(FinishedJob->JobCacheRef);

	// see if there are outstanding jobs that also need to be resolved
	TArray<FShaderCommonCompileJob*> FinishedDuplicateJobs;

	{
		FWriteScopeLock JobLocker(JobLock);

		FShaderCommonCompileJob* CurHead = JobData.DuplicateJobsWaitList;
		while (CurHead)
		{
			checkf(CurHead != FinishedJob, TEXT("Job that is being added to cache was also on a waiting list! Error in bookkeeping."));

			// Need to add these to a list, and process them outside the JobLock scope.  ProcessFinishedJob locks CompileQueueSection,
			// and we don't want to lock that inside a block that also locks JobLock, as it can cause a deadlock given that other
			// code paths obtain the locks in the opposite order.  This is also good for perf, as it avoids holding the lock during
			// the relatively costly SerializeOutput.
			FinishedDuplicateJobs.Add(CurHead);

			// This needs to happen inside the JobLocker scope
			RemoveDuplicateJob(CurHead);

			CurHead = CurHead->NextLink;
		}

		JobData.DuplicateJobsWaitList = nullptr;

		if (FinishedJob->bSucceeded && !FinishedJob->bBypassCache)
		{
			const bool bAddToDDC = GShaderCompilerPerShaderDDCGlobal || !(FinishedJob->bIsDefaultMaterial || FinishedJob->bIsGlobalShader);
			// we only cache jobs that succeded
			AddJobOutput(JobData, FinishedJob, InputHash, SaveContext, FinishedDuplicateJobs.Num(), bAddToDDC);
		}

		// remove ourselves from the jobs in flight
		JobData.ResetJobInFlight();
		FinishedJob->JobCacheRef.Clear();
	}

	if (FinishedDuplicateJobs.Num())
	{
		UE_LOGF(LogShaderCompilers, UE_SHADERCACHE_LOG_LEVEL, "Processed %d outstanding jobs with the same ihash %ls.", FinishedDuplicateJobs.Num(), *LexToString(InputHash));

		check(SaveContext.HasData());
		// Construct a single load context pointing to the data in the save context used above
		FShaderCacheLoadContext LoadContext(SaveContext.ShaderObjectData, SaveContext.ShaderCode, SaveContext.ShaderSymbols);
		for (FShaderCommonCompileJob* DuplicateJob : FinishedDuplicateJobs)
		{
			// reuse the same load context for each duplicate job to avoid reallocating anything
			LoadContext.Reuse();
			DuplicateJob->SerializeOutput(LoadContext);
			checkf(DuplicateJob->bSucceeded == FinishedJob->bSucceeded, TEXT("Different success status for the job with the same ihash"));

			// finish the job instantly
			ProcessFinishedJob(DuplicateJob, FinishedJob->JobStatusPtr->GetStatus());
		}
	}
}

int32 FShaderJobCache::GetNumPendingJobs(EShaderCompileJobPriority InPriority) const
{
	return NumPendingJobs[(int32)InPriority];
}

int32 FShaderJobCache::GetNumOutstandingJobs() const
{
	return NumOutstandingJobs.GetValue();
}

int32 FShaderJobCache::GetNumPendingJobs() const
{
	FReadScopeLock Locker(JobLock);
	int32 NumJobs = 0;
	for (int32 i = 0; i < NumShaderCompileJobPriorities; ++i)
	{
		NumJobs += NumPendingJobs[i];
	}
	return NumJobs;
}

int32 FShaderJobCache::GetPendingJobs(EShaderCompilerWorkerType InWorkerType, EShaderCompileJobPriority InPriority, int32 MinNumJobs, int32 MaxNumJobs, TArray<FShaderCommonCompileJobPtr>& OutJobs)
{
	check(InWorkerType != EShaderCompilerWorkerType::None);
	check(InPriority != EShaderCompileJobPriority::None);

	const int32 PriorityIndex = (int32)InPriority;
	int32 NumPendingJobsOfPriority = 0;
	{
		FReadScopeLock Locker(JobLock);
		NumPendingJobsOfPriority = NumPendingJobs[PriorityIndex].load();
	}

	if (NumPendingJobsOfPriority < MinNumJobs)
	{
		// Not enough jobs
		return 0;
	}

	FWriteScopeLock Locker(JobLock);

	// there was a time window before we checked and then acquired the write lock - make sure the number is still sufficient
	NumPendingJobsOfPriority = NumPendingJobs[PriorityIndex].load();
	if (NumPendingJobsOfPriority < MinNumJobs)
	{
		// Not enough jobs
		return 0;
	}

	OutJobs.Reserve(OutJobs.Num() + FMath::Min(MaxNumJobs, NumPendingJobsOfPriority));
	int32 NumJobs = FMath::Min(MaxNumJobs, NumPendingJobsOfPriority);
	FShaderCommonCompileJobIterator It(PendingJobsHead[PriorityIndex]);
	// Randomize job selection by randomly skipping over jobs while traversing the list.
	// Say, we need to pick 3 jobs out of 5 total. We can skip over 2 jobs in total, e.g. like this:
	// pick one (4 more to go and we need to get 2 of 4), skip one (3 more to go, picking 2 out of 3), pick one (2 more to go, picking 1 of 2), skip one, pick one.
	// It is possible that we won't skip at all and instead pick consequential jobs
	int32 MaxJobsWeCanSkipOver = NumPendingJobsOfPriority - NumJobs;
	for (int32 i = 0; i < NumJobs; ++i)
	{
		FShaderCommonCompileJob& Job = *It;

		// Don't register jobs that have already been registered as assigned to a remote worker and then be transferred to a local worker
		if (!Job.bTransferred)
		{
			GShaderCompilerStats->RegisterAssignedJob(Job);
		}
		ensure(Job.bInputHashSet);

		It.Next();

		check((int32)Job.PendingPriority == PriorityIndex);
		UnlinkJobWithPriority(Job);

		Job.CurrentWorker = InWorkerType;
		Job.UpdateStatus(EShaderCompileJobStatus::Pending, (InWorkerType == EShaderCompilerWorkerType::Distributed) ? EShaderCompileJobCompletionMethod::DistributedExecution : EShaderCompileJobCompletionMethod::LocalExecution);
		OutJobs.Add(&Job);

		// get a random number of jobs to skip (if we can). We're skipping after taking the first job so we can ensure that we always take the latest job into the batch
		if (MaxJobsWeCanSkipOver > 0 && InPriority < EShaderCompileJobPriority::High)
		{
			int32 NumJobsToSkipOver = FMath::RandHelper(MaxJobsWeCanSkipOver + 1);
			while (NumJobsToSkipOver > 0 && It)
			{
				It.Next();
				--NumJobsToSkipOver;
				--MaxJobsWeCanSkipOver;
			}
			checkf(MaxJobsWeCanSkipOver >= 0, TEXT("We skipped over too many jobs"));
			checkf(MaxJobsWeCanSkipOver <= NumPendingJobsOfPriority - i, TEXT("Number of jobs to skip should stay less or equal than the number of nodes to go"));
		}
	}

	return NumJobs;
}

FShaderCacheLoadContext FShaderJobCache::FindOrAdd(const FShaderCompilerInputHash& Hash, FShaderCommonCompileJob* Job, const bool bCheckDDC)
{
	LLM_SCOPE_BYTAG(ShaderCompiler);

	++TotalSearchAttempts;
	TRACE_COUNTER_INCREMENT(Shaders_JobCacheSearchAttempts);
	TRACE_CPUPROFILER_EVENT_SCOPE(FShaderJobCache::Find);
	uint64 InputHashToJobDataSize = InputHashToJobData.GetAllocatedSize();

	Job->JobCacheRef = InputHashToJobData.FindOrAdd(Hash);
	FShaderJobData& JobData = GetShaderJobData(Job->JobCacheRef);

	CurrentlyAllocatedMemory += InputHashToJobData.GetAllocatedSize() - InputHashToJobDataSize;

	if (JobData.HasOutput())
	{
		++TotalCacheHits;
		TRACE_COUNTER_INCREMENT(Shaders_JobCacheHits);

		FStoredOutput** CannedOutput = Outputs.Find(JobData.OutputHash);
		// we should not allow a dangling input to output mapping to exist
		checkf(CannedOutput != nullptr, TEXT("Inconsistency in FShaderJobCache - cache record for ihash %s (data 0x%p) exists, but output %s (%s) cannot be found."),
			*LexToString(Hash), &JobData, *LexToString(JobData.OutputHash), JobData.bOutputFromDDC ? TEXT("DDC") : TEXT("Job"));
		// update the output hit count
		(*CannedOutput)->NumHits++;

		return FShaderCacheLoadContext((*CannedOutput)->JobOutput, (*CannedOutput)->JobCode, (*CannedOutput)->JobSymbols);
	}
#if WITH_EDITOR
	else
	{
		// If NoShaderDDC then don't check for a material the first time we encounter it to simulate a cold DDC
		static bool bNoShaderDDC = FParse::Param(FCommandLine::Get(), TEXT("noshaderddc"));

		// If we didn't find it in memory search the DDC if it's enabled.
		// Don't search if this isn't the first job with this hash (JobInFlight already set), or there's already a request in flight.
		const bool bCachePerShaderDDC = IsShaderJobCacheDDCEnabled() && bCheckDDC && !bNoShaderDDC;
		if (bCachePerShaderDDC && (JobData.JobInFlight == nullptr) && !Job->RequestOwner)
		{
			TRACE_COUNTER_INCREMENT(Shaders_JobCacheDDCRequests);

			++TotalCacheDDCQueries;

			UE::DerivedData::EPriority DerivedDataPriority;
			UE::DerivedData::FRequestOwner* RequestOwner;

			Job->UpdateStatus(EShaderCompileJobStatus::PendingDDC);

			static const bool PerShaderDDCAsync = CVarShaderCompilerPerShaderDDCAsync.GetValueOnAnyThread();
			if (PerShaderDDCAsync && FGenericPlatformProcess::SupportsMultithreading())
			{
				if (IsRunningCookCommandlet())
				{
					DerivedDataPriority = UE::DerivedData::EPriority::Highest;
				}
				else
				{
					switch (Job->Priority)
					{
					case EShaderCompileJobPriority::Low:		DerivedDataPriority = UE::DerivedData::EPriority::Low;		break;
					case EShaderCompileJobPriority::Normal:		DerivedDataPriority = UE::DerivedData::EPriority::Normal;	break;
					default:									DerivedDataPriority = UE::DerivedData::EPriority::Highest;	break;
					}
				}
				Job->RequestOwner = MakePimpl<UE::DerivedData::FRequestOwner>(DerivedDataPriority);
				RequestOwner = Job->RequestOwner.Get();
			}
			else
			{
				DerivedDataPriority = UE::DerivedData::EPriority::Blocking;
				RequestOwner = new UE::DerivedData::FRequestOwner(DerivedDataPriority);
			}

			UE::DerivedData::FCacheGetRequest Request;
			Request.Name = TEXT("FShaderJobCache");
			// Create key.
			Request.Key.Bucket = ShaderJobCacheDDCBucket;
			Request.Key.Hash = Hash;
			Request.Policy = IsShaderJobCacheDDCRemotePolicyEnabled() ? UE::DerivedData::ECachePolicy::Default : UE::DerivedData::ECachePolicy::Local;

			bool bCompletedSynchronously = false;
			bool* bCompletedSynchronouslyPtr = &bCompletedSynchronously;
			
			// Optionally read the cached output back to the main thread (this is only ever set if bCompletedSynchronously is true)
			FStoredOutput* StoredOutput = nullptr;
			FStoredOutput** StoredOutputPtr = &StoredOutput;

			UE::DerivedData::GetCache().Get(
				{ Request },
				*RequestOwner,
				[this, JobDataPtr = &JobData, DerivedDataPriority, StoredOutputPtr, bCompletedSynchronouslyPtr](UE::DerivedData::FCacheGetResponse&& Response)
				{
					if (GShaderCompilerDebugStallDDCQuery > 0)
					{
						FPlatformProcess::Sleep(GShaderCompilerDebugStallDDCQuery * 0.001f);
					}

					bool bIsAsync = (DerivedDataPriority != UE::DerivedData::EPriority::Blocking);

					// Check thread local variable to see if we're in the submit job thread (DDC request completing synchronously), in which
					// case we want to go through the synchronous code paths below, instead of async.
					if (bInSubmitJobThread)
					{
						bIsAsync = false;
						*bCompletedSynchronouslyPtr = true;
					}

					if (Response.Status == UE::DerivedData::EStatus::Ok)
					{
						// Retrieve the shared buffer containing the job output and compute the associated output hash for the result retrieved from DDC
						// If an existing duplicate of this buffer is already registered in the Outputs map, this copy will be freed at end of scope
						FShaderCacheLoadContext LoadContext;
						LoadContext.ReadFromRecord(Response.Record);
						FJobOutputHash OutputHash = ComputeJobHash(LoadContext);

						TRACE_COUNTER_ADD(Shaders_JobCacheDDCBytesReceived, LoadContext.GetSerializedSize());
						TRACE_COUNTER_INCREMENT(Shaders_JobCacheDDCHits);

						// If we are running the cache logic async (not blocking in the main thread), we need a lock before writing to the job cache.
						// Otherwise, the lock will already be held by the main thread (and trying to lock here would just deadlock).
						if (bIsAsync)
						{
							JobLock.WriteLock();
							check(JobDataPtr->JobInFlight);

							// If job was cancelled, it will have been unlinked from PendingSubmitJobTaskJobs, and we can ignore the results.
							if (!JobDataPtr->JobInFlight->PrevLink)
							{
								UE_LOGF(LogShaderCompilers, Display, "Cancelled job 0x%p (data 0x%p) with pending DDC hit.", JobDataPtr->JobInFlight.GetReference(), JobDataPtr);
								if (JobDataPtr->JobInFlight)
								{
									JobDataPtr->JobInFlight->UpdateStatus(EShaderCompileJobStatus::Cancelled);
									JobDataPtr->ResetJobInFlight();
								}
								JobLock.WriteUnlock();
								return;
							}
							else
							{
								Unlink(*JobDataPtr->JobInFlight);		// from PendingSubmitJobTaskJobs
							}
						}

						// Add a DDC hit
						++TotalCacheDDCHits;

						FStoredOutput** ExistingStoredOutput = Outputs.Find(OutputHash);
						FStoredOutput* StoredOutput = ExistingStoredOutput ? *ExistingStoredOutput : nullptr;
						if (StoredOutput == nullptr)
						{
							// Create a new entry to store in the FShaderJobCache if one doesn't already exist for this output hash
							check(LoadContext.HasData()); // sanity check that the load context was populated properly
							StoredOutput = new FStoredOutput();
							StoredOutput->JobOutput = LoadContext.ShaderObjectData;
							LoadContext.MoveCode(StoredOutput->JobCode, StoredOutput->JobSymbols);
							Outputs.Add(OutputHash, StoredOutput);
							CurrentlyAllocatedMemory += StoredOutput->GetAllocatedSize();
						}

						// Increment refcount of output whether or not we created it above
						StoredOutput->AddRef();

						JobDataPtr->OutputHash = OutputHash;
						JobDataPtr->bOutputFromDDC = true;

						// If async, add processed results to output.  For the synchronous case, this is handled back in the main thread.
						if (bIsAsync)
						{
							check(JobDataPtr->JobInFlight);
							FShaderCommonCompileJobPtr Job = JobDataPtr->JobInFlight;

							UE_LOGF(LogShaderCompilers, UE_SHADERCACHE_LOG_LEVEL, "Found an async DDC result for job with ihash %ls.", *LexToString(Job->InputHash));

							// Get list of finished jobs -- JobInFlight, plus any duplicates -- and clear the job cache data
							TArray<FShaderCommonCompileJob*> FinishedJobs;
							FinishedJobs.Add(Job);

							FShaderCommonCompileJob* CurHead = JobDataPtr->DuplicateJobsWaitList;
							while (CurHead)
							{
								FinishedJobs.Add(CurHead);
								RemoveDuplicateJob(CurHead);
								CurHead = CurHead->NextLink;
							}
							JobDataPtr->DuplicateJobsWaitList = nullptr;
							JobDataPtr->ResetJobInFlight();
							Job->JobCacheRef.Clear();

							// Need to release the lock before calling ProcessFinishedJobs
							JobLock.WriteUnlock();

							// Re-initialize load context pointing to the stored code array in the cache and reuse for each duplicate job needing population
							LoadContext.Reset(StoredOutput->JobOutput, StoredOutput->JobCode, StoredOutput->JobSymbols);

							// Call ProcessFinishedJob on main job and duplicates
							for (FShaderCommonCompileJob* FinishedJob : FinishedJobs)
							{
								LoadContext.Reuse();
								FinishedJob->SerializeOutput(LoadContext);
								EShaderCompileJobCompletionMethod Method = FinishedJob->JobStatusPtr->GetCompletionMethod();
								// We only update execution method to "DdcHit" for the first job in the chain (of which all subsequent entries are dupes) for tracking purposes
								// Duplicate jobs will have previously had their execution method set to "Duplicate" when queued, so this maintains that
								if (Method == EShaderCompileJobCompletionMethod::Unset)
								{
									Method = EShaderCompileJobCompletionMethod::DdcHit;
								}
								ProcessFinishedJob(FinishedJob, EShaderCompileJobStatus::Completed, Method);
							}

							if (FinishedJobs.Num() > 1)
							{
								UE_LOGF(LogShaderCompilers, UE_SHADERCACHE_LOG_LEVEL, "Processed %d outstanding jobs with the same ihash %ls.", FinishedJobs.Num() - 1, *LexToString(Job->InputHash));
							}
						}
						else
						{
							// Send results back to the main thread when running synchronous
							*StoredOutputPtr = StoredOutput;
						}
					}
					else
					{
						// If async, add job to pending queue.  For the synchronous case, this is handled back in the main thread.
						if (bIsAsync)
						{
							FWriteScopeLock Locker(JobLock);
							FShaderCommonCompileJob* Job = JobDataPtr->JobInFlight;
							check(Job);

							// If job was cancelled, it will have been unlinked from PendingSubmitJobTaskJobs, and we can ignore it.
							if (!Job->PrevLink)
							{
								UE_LOGF(LogShaderCompilers, Display, "Cancelled job 0x%p (data 0x%p) with pending DDC miss.", Job, JobDataPtr);
								Job->UpdateStatus(EShaderCompileJobStatus::Cancelled);
								JobDataPtr->ResetJobInFlight();
								return;
							}
							else
							{
								Unlink(*Job);		// from PendingSubmitJobTaskJobs
							}

							LinkJobWithPriority(*Job);
						}
					}
				});

			// For blocking requests, wait on the results, and delete the request
			if (RequestOwner->GetPriority() == UE::DerivedData::EPriority::Blocking)
			{
				RequestOwner->Wait();
				delete RequestOwner;
			}
			else if (bCompletedSynchronously)
			{
				// It's also possible (notably when DDC verification is enabled) for the request to have completed synchronously,
				// in which case we can delete the TPimplPtr request owner by setting it to null.  This tells the main thread
				// there is no async DDC request in flight, and it should handle adding the pending job to the queue, since the
				// DDC request callback won't be handling that.
				Job->RequestOwner = nullptr;

				if (StoredOutput)
				{
					return FShaderCacheLoadContext(StoredOutput->JobOutput, StoredOutput->JobCode, StoredOutput->JobSymbols);
				}
			}
		}
	}
#endif
	return FShaderCacheLoadContext();
}

FShaderJobData* FShaderJobCache::Find(const FShaderCompilerInputHash& Hash)
{
	return InputHashToJobData.Find(Hash);
}

/** Adds a reference to a duplicate job (to the DuplicateJobs array) */
void FShaderJobCache::AddDuplicateJob(FShaderCommonCompileJob* DuplicateJob)
{
	check(DuplicateJob->JobCacheRef.DuplicateIndex == INDEX_NONE);

	DuplicateJob->JobCacheRef.DuplicateIndex = DuplicateJobs.Add(DuplicateJob);
}

/** Removes a reference to a duplicate job (from the DuplicateJobs array)  */
void FShaderJobCache::RemoveDuplicateJob(FShaderCommonCompileJob* DuplicateJob)
{
	int32 DuplicateIndex = DuplicateJob->JobCacheRef.DuplicateIndex;
	check(DuplicateIndex >= 0 && DuplicateIndex < DuplicateJobs.Num() && DuplicateJobs[DuplicateIndex] == DuplicateJob);
	DuplicateJob->JobCacheRef.DuplicateIndex = INDEX_NONE;

	DuplicateJobs.RemoveAtSwap(DuplicateIndex);

	// After removing, we need to update the cached index of the job we swapped
	if (DuplicateIndex < DuplicateJobs.Num())
	{
		DuplicateJobs[DuplicateIndex]->JobCacheRef.DuplicateIndex = DuplicateIndex;
	}
}

uint64 FShaderJobCache::GetCurrentMemoryBudget() const
{
	uint64 AbsoluteLimit = static_cast<uint64>(GShaderCompilerMaxJobCacheMemoryMB) * 1024ULL * 1024ULL;
	uint64 RelativeLimit = FMath::Clamp(static_cast<double>(GShaderCompilerMaxJobCacheMemoryPercent), 0.0, 100.0) * (static_cast<double>(FPlatformMemory::GetPhysicalGBRam()) * 1024 * 1024 * 1024) / 100.0;
	return FMath::Min(AbsoluteLimit, RelativeLimit);
}

FShaderJobCache::FShaderJobCache(FCriticalSection& InCompileQueueSection)
	: CompileQueueSection(InCompileQueueSection)
	, AllSubmitJobsCompleteEvent(EEventMode::ManualReset)
{
	FMemory::Memzero(PendingJobsHead);
	FMemory::Memzero(NumPendingJobs);
#if UE_SHADERCOMPILER_FIFO_JOB_EXECUTION
	for (int32 PriorityIndex = 0; PriorityIndex < NumShaderCompileJobPriorities; PriorityIndex++)
	{
		PendingJobsTail[PriorityIndex] = &PendingJobsHead[PriorityIndex];
	}
#endif

	CurrentlyAllocatedMemory = sizeof(*this) + InputHashToJobData.GetAllocatedSize() + Outputs.GetAllocatedSize();
}

FShaderJobCache::~FShaderJobCache()
{
	AllSubmitJobsCompleteEvent->Reset();
	if (NumPendingSubmitJobs.load(std::memory_order_acquire) > 0)
	{
		AllSubmitJobsCompleteEvent->Wait();
	}

	for (TMap<FJobOutputHash, FStoredOutput*>::TIterator Iter(Outputs); Iter; ++Iter)
	{
		delete Iter.Value();
	}
}

void FShaderJobCache::AddJobOutput(FShaderJobData& JobData, const FShaderCommonCompileJob* FinishedJob, const FShaderCompilerInputHash& Hash, FShaderCacheSaveContext& SaveContext, int32 InitialHitCount, const bool bAddToDDC)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FShaderJobCache::Add);

	if (JobData.HasOutput() && !ShaderCompiler::IsJobCacheDebugValidateEnabled())
	{
		return;
	}

	FJobOutputHash OutputHash = ComputeJobHash(SaveContext);

	if (JobData.HasOutput() && ShaderCompiler::IsJobCacheDebugValidateEnabled())
	{
		if (OutputHash != JobData.OutputHash)
		{
			TStringBuilder<1024> FinishedJobName;
			FinishedJob->AppendDebugName(FinishedJobName);

			const FString* CachedJobName = CachedJobNames.Find(JobData.OutputHash);
			check(CachedJobName);
			UE_LOGF(
				LogShaderCompilers,
				Warning,
				"Job cache validation found output mismatch!\n" "Cached job: %ls\n" "Original job: %ls\n",
				**CachedJobName, FinishedJobName.ToString());

			if (ShaderCompiler::IsDumpShaderDebugInfoAlwaysEnabled())
			{
				static bool bOnce = false;
				if (!bOnce)
				{
					UE_LOGF(
						LogShaderCompilers,
						Warning,
						"Enable r.DumpShaderDebugInfo=1 to get debug info paths for the mismatching jobs instead of group names (to allow diffing debug artifacts)");
					bOnce = true;
				}
			}
		}
		return;
	}

	const bool bDumpCachedDebugInfo = CVarDumpShaderOutputCacheHits.GetValueOnAnyThread();

	// Get dump shader debug output path
	FString InputDebugInfoPath, InputSourceFilename;
	if (bDumpCachedDebugInfo)
	{
		if (const FShaderCompileJob* SingleJob = FinishedJob->GetSingleShaderJob())
		{
			const FShaderCompilerInput& Input = SingleJob->Input;
			if (!Input.DumpDebugInfoPath.IsEmpty())
			{
				InputDebugInfoPath = Input.DumpDebugInfoPath;
				InputSourceFilename = FPaths::GetBaseFilename(Input.GetSourceFilename());
			}
		}
	}

	// Cache this value for thread safety
	bool bDiscardCacheOutputs = GShaderCompilerDebugDiscardCacheOutputs != 0;

	// add the record
	if (UNLIKELY(bDiscardCacheOutputs == false))
	{
		JobData.OutputHash = OutputHash;
		JobData.bOutputFromDDC = false;
	}

	FStoredOutput** CannedOutput = Outputs.Find(OutputHash);
	if (CannedOutput)
	{
		// update the output hit count
		int32 NumRef;
		if (UNLIKELY(bDiscardCacheOutputs == false))
		{
			NumRef = (*CannedOutput)->AddRef();
		}
		else
		{
			NumRef = (*CannedOutput)->GetNumReferences();
		}

		if (UNLIKELY(bDumpCachedDebugInfo))
		{
			// Write cache hit debug file
			const FString& CachedDebugInfoPath = (*CannedOutput)->CachedDebugInfoPath;
			if (!CachedDebugInfoPath.IsEmpty())
			{
				const int32 CacheHit = NumRef - 1;
				const FString CacheHitFilename = FString::Printf(TEXT("%s/%s.%d.cachehit"), *CachedDebugInfoPath, *InputSourceFilename, CacheHit);
				FFileHelper::SaveStringToFile(InputDebugInfoPath, *CacheHitFilename);
			}
		}
	}
	else
	{
		if (UNLIKELY(bDiscardCacheOutputs == false))
		{
			const uint64 OutputsOriginalSize = Outputs.GetAllocatedSize();

			check(SaveContext.HasData());
			FStoredOutput* NewStoredOutput = new FStoredOutput();
			NewStoredOutput->NumHits = InitialHitCount;
			NewStoredOutput->JobOutput = SaveContext.ShaderObjectData;
			SaveContext.MoveCode(NewStoredOutput->JobCode, NewStoredOutput->JobSymbols);
			NewStoredOutput->CachedDebugInfoPath = InputDebugInfoPath;
			NewStoredOutput->AddRef();
			Outputs.Add(OutputHash, NewStoredOutput);

			if (ShaderCompiler::IsJobCacheDebugValidateEnabled())
			{
				TStringBuilder<1024> NameBuilder;
				FinishedJob->AppendDebugName(NameBuilder);

				CachedJobNames.Add(OutputHash, NameBuilder.ToString());
			}

			CurrentlyAllocatedMemory += NewStoredOutput->GetAllocatedSize() + Outputs.GetAllocatedSize() - OutputsOriginalSize;
		}

		if (UNLIKELY(bDumpCachedDebugInfo))
		{
			// Write new allocated cache file
			if (!InputDebugInfoPath.IsEmpty())
			{
				const FString CacheOutputFilename = FString::Printf(TEXT("%s/%s.joboutput"), *InputDebugInfoPath, *InputSourceFilename);
				FFileHelper::SaveArrayToFile(TArrayView<const uint8>((const uint8*)SaveContext.ShaderObjectData.GetData(), SaveContext.ShaderObjectData.GetSize()), *CacheOutputFilename);
				for (int32 CodeIndex = 0; CodeIndex < SaveContext.ShaderCode.Num(); ++CodeIndex)
				{
					const FString CacheCodeFilename = FString::Printf(TEXT("%s/%s_%d.bytecode"), *InputDebugInfoPath, *InputSourceFilename, CodeIndex);
					const FCompositeBuffer& JobCode = SaveContext.ShaderCode[CodeIndex];
					check(JobCode.GetSegments().Num() == 2); // first segment is header, second is actual code
					FSharedBuffer CodeBuffer = JobCode.GetSegments()[1];
					FFileHelper::SaveArrayToFile(TArrayView<const uint8>((const uint8*)CodeBuffer.GetData(), CodeBuffer.GetSize()), *CacheCodeFilename);
				}
			}
		}

		// delete oldest cache entries if we exceed the budget
		uint64 MemoryBudgetBytes = GetCurrentMemoryBudget();
		if (MemoryBudgetBytes)
		{
			if (CurrentlyAllocatedMemory > MemoryBudgetBytes)
			{
				TRACE_CPUPROFILER_EVENT_SCOPE(FShaderJobCache::Trim);

				uint64 TargetBudgetBytes = MemoryBudgetBytes * FMath::Clamp(GShaderCompilerJobCacheOverflowReducePercent, 0, 100) / 100;
				uint64 MemoryBefore = CurrentlyAllocatedMemory;

				// Cull outputs to reach the budget target
				CullOutputsToMemoryBudget(TargetBudgetBytes);

				UE_LOGF(LogShaderCompilers, Display, "Memory overflow, reduced from %.1lf to %.1lf MB.", (double)MemoryBefore / (1024 * 1024), (double)CurrentlyAllocatedMemory / (1024 * 1024));
			}
		}
	}

#if WITH_EDITOR
	const bool bCachePerShaderDDC = IsShaderJobCacheDDCEnabled() && bAddToDDC;

	if (bCachePerShaderDDC)
	{
		// Create key.
		UE::DerivedData::FCacheKey Key;
		Key.Bucket = ShaderJobCacheDDCBucket;
		Key.Hash = Hash;

		UE::DerivedData::FRequestOwner RequestOwner(UE::DerivedData::EPriority::Normal);
		UE::DerivedData::FRequestBarrier RequestBarrier(RequestOwner);
		RequestOwner.KeepAlive();
		UE::DerivedData::GetCache().Put(
			{ {{TEXT("FShaderJobCache")}, SaveContext.BuildCacheRecord(Key), IsShaderJobCacheDDCRemotePolicyEnabled() ? UE::DerivedData::ECachePolicy::Default : UE::DerivedData::ECachePolicy::Local } },
			RequestOwner
		);


		TRACE_COUNTER_ADD(Shaders_JobCacheDDCBytesSent, SaveContext.GetSerializedSize());
	}
#endif


}

/** Returns memory used by the cache*/
uint64 FShaderJobCache::GetAllocatedMemory() const
{
	return CurrentlyAllocatedMemory;
}

/** Compute memory used by the cache from scratch.  Should match GetAllocatedMemory() if CurrentlyAllocateMemory is being properly updated. */
uint64 FShaderJobCache::ComputeAllocatedMemory() const
{
	uint64 AllocatedSize = sizeof(FShaderJobCache) + InputHashToJobData.GetAllocatedSize() + Outputs.GetAllocatedSize();
	for (auto OutputIter : Outputs)
	{
		AllocatedSize += OutputIter.Value->GetAllocatedSize();
	}
	return AllocatedSize;
}

void FShaderJobCache::GetStats(FShaderCompilerStats& OutStats) const
{
	FReadScopeLock Locker(JobLock);
	OutStats.Counters.TotalCacheSearchAttempts = TotalSearchAttempts;
	OutStats.Counters.TotalCacheHits = TotalCacheHits;
	OutStats.Counters.TotalCacheDuplicates = TotalCacheDuplicates;
	OutStats.Counters.TotalCacheDDCQueries = TotalCacheDDCQueries;
	OutStats.Counters.TotalCacheDDCHits = TotalCacheDDCHits;
	OutStats.Counters.UniqueCacheInputHashes = InputHashToJobData.Num();
	OutStats.Counters.UniqueCacheOutputs = Outputs.Num();
	OutStats.Counters.CacheMemUsed = GetAllocatedMemory();
	OutStats.Counters.CacheMemBudget = GetCurrentMemoryBudget();
}


