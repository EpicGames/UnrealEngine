// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	ShaderCompilerMemoryLimit.cpp: Wrapper for Windows specific Process Tree functionality.
=============================================================================*/

#include "ShaderCompilerMemoryLimit.h"
#include "HAL/ConsoleManager.h"

#if PLATFORM_WINDOWS

#include "Windows/MinimalWindowsApi.h"
#include "Windows/WindowsHWrapper.h"
#include "Windows/AllowWindowsPlatformTypes.h"
	#include <Psapi.h>
#include "Windows/HideWindowsPlatformTypes.h"

void FWindowsResourceProcessTreeMemory::AddRootProcessId(uint32 ProcessId)
{
	if (ProcessId != 0 )
	{
		RootProcessIds.Add(ProcessId);
	}
}

void FWindowsResourceProcessTreeMemory::Reset()
{
	RootProcessIds.Empty();

	if (!bCachedSystemProcessIds)
		AllProcesses.Empty();

	TreeProcessIds.Empty();
}

void FWindowsResourceProcessTreeMemory::CacheSystemProcessIds()
{
	if (!bCachedSystemProcessIds)
	{
		RebuildSystemProcessIdList();
	}
	bCachedSystemProcessIds = true;
}

void FWindowsResourceProcessTreeMemory::ClearSystemProcessIdsCache()
{
	AllProcesses.Empty();
	bCachedSystemProcessIds = false;
}

bool FWindowsResourceProcessTreeMemory::TryGetMemoryUsage(FPlatformProcessMemoryStats& OutStats)
{
	CollectTreeProcessIds();

	OutStats = { };

	for (const uint32 ProcessId : TreeProcessIds)
	{
		HANDLE ProcessHandle = ::OpenProcess(PROCESS_QUERY_INFORMATION | PROCESS_VM_READ, false, ProcessId);

		if (ProcessHandle != NULL)
		{
			PROCESS_MEMORY_COUNTERS ProcessMemoryCounters;
			FPlatformMemory::Memzero(&ProcessMemoryCounters, sizeof(ProcessMemoryCounters));

			if (!::GetProcessMemoryInfo(ProcessHandle, &ProcessMemoryCounters, sizeof(ProcessMemoryCounters)))
			{
				if (FWindowsPlatformMisc::IsWine() && ::GetLastError() == ERROR_ACCESS_DENIED)
				{
					// Under Wine, GetProcessMemoryInfo() incorrectly returns ERROR_ACCESS_DENIED if a process is no longer running.
				}
				else
				{
					UE_LOGF(LogWindows, Warning, "Failure in call to GetProcessMemoryInfo (GetLastError=%d)", ::GetLastError());
				}

				::CloseHandle(ProcessHandle);
				return false;
			}
			else
			{
				OutStats.UsedPhysical += ProcessMemoryCounters.WorkingSetSize;
				OutStats.PeakUsedPhysical += ProcessMemoryCounters.PeakWorkingSetSize;
				OutStats.UsedVirtual += ProcessMemoryCounters.PagefileUsage;
				OutStats.PeakUsedVirtual += ProcessMemoryCounters.PeakPagefileUsage;
			}

			::CloseHandle(ProcessHandle);
		}
	}

	return true;
}

void FWindowsResourceProcessTreeMemory::CollectTreeProcessIds()
{
	if (!bCachedSystemProcessIds)
		AllProcesses.Empty();
	TreeProcessIds.Empty();

	if (!RootProcessIds.IsEmpty())
	{
		RebuildSystemProcessIdList();

		for (const uint32 RootProcessId : RootProcessIds)
		{
			CollectTreeProcessIdsRecurse(RootProcessId);
		}
	}
}

void FWindowsResourceProcessTreeMemory::CollectTreeProcessIdsRecurse(uint32 RootProcessId)
{
	if (RootProcessId == 0)
	{
		return;
	}

	bool AlreadyVisited = false;
	TreeProcessIds.Add(RootProcessId, &AlreadyVisited);

	if (AlreadyVisited)
	{
		return;
	}

	for (const FProcessInfo& JobProcessInfo : AllProcesses)
	{
		if (JobProcessInfo.ParentProcessId == RootProcessId)
		{
			CollectTreeProcessIdsRecurse(JobProcessInfo.ProcessId);
		}
	}
}

void FWindowsResourceProcessTreeMemory::RebuildSystemProcessIdList()
{
	// Dont rebuild if we already cached
	if (bCachedSystemProcessIds)
		return;

	AllProcesses.Empty();

	FPlatformProcess::FProcEnumerator ProcEnumerator;
	while (ProcEnumerator.MoveNext())
	{
		FPlatformProcess::FProcEnumInfo ProcEnumInfo = ProcEnumerator.GetCurrent();

		FProcessInfo ProcessInfo;
		ProcessInfo.ProcessId = ProcEnumInfo.GetPID();
		ProcessInfo.ParentProcessId = ProcEnumInfo.GetParentPID();

		AllProcesses.Push(ProcessInfo);
	}
}

#else

void FGenericResourceProcessTreeMemory::AddRootProcessId(uint32 ProcessId)
{
	// dummy
}

void FGenericResourceProcessTreeMemory::Reset()
{
	// dummy
}

void FGenericResourceProcessTreeMemory::CacheSystemProcessIds()
{
	// dummy
}

void FGenericResourceProcessTreeMemory::ClearSystemProcessIdsCache()
{
	// dummy
}

bool FGenericResourceProcessTreeMemory::TryGetMemoryUsage(FPlatformProcessMemoryStats& OutStats)
{
	return false; // dummy
}

#endif // PLATFORM_WINDOWS

