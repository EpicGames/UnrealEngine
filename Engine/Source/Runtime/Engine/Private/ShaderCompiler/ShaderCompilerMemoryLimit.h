// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	ShaderCompilerMemoryLimit.h: Wrapper for Windows specific Process Tree functionality.
=============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "HAL/PlatformProcess.h"

#if PLATFORM_WINDOWS

/** Wrapper for measuring memory of a process tree. This is only available on Windows and acts as a placeholder on other platforms. */
class FWindowsResourceProcessTreeMemory
{
public:
	FWindowsResourceProcessTreeMemory(const FWindowsResourceProcessTreeMemory&) = delete;
	FWindowsResourceProcessTreeMemory& operator = (const FWindowsResourceProcessTreeMemory&) = delete;

	FWindowsResourceProcessTreeMemory() = default;
	~FWindowsResourceProcessTreeMemory() = default;

	/** Add a ProcessId as the root of a set of processess to measure. */
	void AddRootProcessId(uint32 ProcessId);

	/** Empty the list of ProcessIds to measure */
	void Reset();

	/** Caches build of system process ids. Saves overhead if calling TryGetMemoryUsage on separate root process Ids */
	void CacheSystemProcessIds();

	/** Clears the cache of system process Ids. */
	void ClearSystemProcessIdsCache();

	/** Try and get the total memory used by all process and their children from AddRootProcessId. */
	bool TryGetMemoryUsage(FPlatformProcessMemoryStats& OutStats);

private:
	struct FProcessInfo {
		uint32 ProcessId = 0;
		uint32 ParentProcessId = 0;
	};

	/* The Process IDs we will measure */
	TSet<uint32> RootProcessIds;

	/* All Processes in the system */
	TArray<FProcessInfo> AllProcesses;

	/* The root Process IDs and all child Process IDs */
	TSet<uint32> TreeProcessIds;

	/* State of cached system process Ids */
	bool bCachedSystemProcessIds = false;

	/* Populate the TreeProcessIds with RootProcessIds and their children. */
	void CollectTreeProcessIds();

	/* Recursively add the RootProcessId and children to the TreeProcessIds */
	void CollectTreeProcessIdsRecurse(uint32 RootProcessId);

	/* Populate the system process Id list */
	void RebuildSystemProcessIdList();
};

using FResourceProcessTreeMemory = FWindowsResourceProcessTreeMemory;

#else

/** Wrapper for measuring memory of a process tree. This is only available on Windows and acts as a placeholder on other platforms. */
class FGenericResourceProcessTreeMemory
{
public:
	FGenericResourceProcessTreeMemory(const FGenericResourceProcessTreeMemory&) = delete;
	FGenericResourceProcessTreeMemory& operator = (const FGenericResourceProcessTreeMemory&) = delete;

	FGenericResourceProcessTreeMemory() = default;
	~FGenericResourceProcessTreeMemory() = default;

	void AddRootProcessId(uint32 ProcessId);
	void Reset();
	void CacheSystemProcessIds();
	void ClearSystemProcessIdsCache();
	bool TryGetMemoryUsage(FPlatformProcessMemoryStats& OutStats);
};

using FResourceProcessTreeMemory = FGenericResourceProcessTreeMemory;

#endif // PLATFORM_WINDOWS
