// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	ShaderCompilerPrivate.h:
	To be included by ShaderCompiler*.cpp files that share various variables and functions.
=============================================================================*/

#pragma once

#include "ShaderCompiler.h"
#include "DerivedDataCache.h"
#include "ProfilingDebugging/CookStats.h"

#include <atomic>

LLM_DECLARE_TAG(ShaderCompiler);

class FGlobalShaderMap;
class FGlobalShaderMapId;

extern FShaderCompilerStats* GShaderCompilerStats;
extern FShaderCompilingManager* GShaderCompilingManager;
extern const ITargetPlatform* GGlobalShaderTargetPlatform[SP_NumPlatforms];
extern FGlobalShaderMap* GGlobalShaderMap_DeferredDeleteCopy[SP_NumPlatforms];

bool AreShaderErrorsFatal();
FString GetGlobalShaderMapKeyString(const FGlobalShaderMapId& ShaderMapId, EShaderPlatform Platform, TArray<FShaderTypeDependency> const& Dependencies);

UE::DerivedData::FCacheKey GetGlobalShaderMapKey(const FGlobalShaderMapId& ShaderMapId, EShaderPlatform Platform, const ITargetPlatform* TargetPlatform, TArray<FShaderTypeDependency> const& Dependencies);
UE::FSharedString GetGlobalShaderMapName(const FGlobalShaderMapId& ShaderMapId, EShaderPlatform Platform, const FString& Key);

// Configuration to retry shader compile through workers after a worker has been abandoned
static constexpr int32 GSingleThreadedRunsIdle = -1;

#if ENABLE_COOK_STATS
namespace GlobalShaderCookStats
{
	extern FCookStats::FDDCResourceUsageStats UsageStats;
	extern int32 ShadersCompiled;
}
#endif

namespace ShaderCompilerCookStats
{
	extern std::atomic<double> AsyncCompileTimeSec;
}

/** Helper functions for logging more debug info */
namespace ShaderCompiler
{
	FString GetTargetPlatformName(const ITargetPlatform* TargetPlatform);
	bool IsJobCacheDebugValidateEnabled();
	bool IsRemoteCompilingAllowed();
	bool IsDumpShaderDebugInfoAlwaysEnabled();
}

