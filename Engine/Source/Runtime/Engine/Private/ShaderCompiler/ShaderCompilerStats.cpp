// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	ShaderCompilerStats.cpp: Implements FShaderCompilerStats.
=============================================================================*/

#include "ShaderCompilerPrivate.h"

#include "AnalyticsEventAttribute.h"
#include "CoreMinimal.h"
#include "DistributedBuildControllerInterface.h"
#include "Dom/JsonObject.h"
#include "JsonObjectConverter.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/EngineVersion.h"
#include "ProfilingDebugging/CookStats.h"
#include "ProfilingDebugging/DiagnosticTable.h"
#include "ProfilingDebugging/LoadTimeTracker.h"
#include "ProfilingDebugging/StallDetector.h"
#include "RenderingThread.h"
#include "Serialization/CompactBinaryWriter.h"
#include "ShaderCompileWorkerUtil.h"
#include "Misc/FileHelper.h"

static int32 GLogShaderCompilerStats = 0;
static FAutoConsoleVariableRef CVarLogShaderCompilerStats(
	TEXT("r.LogShaderCompilerStats"),
	GLogShaderCompilerStats,
	TEXT("When set to 1, Log detailed shader compiler stats."));

static int32 GMaxShaderStatsToLog = 5;
static FAutoConsoleVariableRef CVarShaderCompilerMaxShaderStatsToPrint(
	TEXT("r.ShaderCompiler.MaxShaderStatsToPrint"),
	GMaxShaderStatsToLog,
	TEXT("Max number of shaders FShaderJobCache stats logs out"),
	ECVF_Default);

static bool GDumpShaderTimeStats = false;
static FAutoConsoleVariableRef CVarDumpShaderTimeStats(
	TEXT("r.ShaderCompiler.DumpShaderTimeStats"),
	GDumpShaderTimeStats,
	TEXT("When set to true, dump shader compiler timing statistics to a CSV file."));

bool GShaderCompilerDumpWorkerDiagnostics = false;
static FAutoConsoleVariableRef CVarDumpWorkerDiagnostics(
	TEXT("r.ShaderCompiler.DumpWorkerDiagnostics"),
	GShaderCompilerDumpWorkerDiagnostics,
	TEXT("If enabled, the shader compiler will dump CSV files into the ShaderDebugInfo folder with diagnostics for each batch of shader compile jobs."),
	ECVF_ReadOnly);


FShaderCompilerStats* GShaderCompilerStats = nullptr;

void FShaderCompilerStats::WriteStats(FOutputDevice* Ar)
{
#if ALLOW_DEBUG_FILES
	constexpr static const TCHAR DebugText[] = TEXT("Wrote shader compile stats to file '%s'.");
	{
		FlushRenderingCommands();

		FString FileName = FPaths::Combine(*FPaths::ProjectSavedDir(),
			FString::Printf(TEXT("MaterialStats/Stats-%s.csv"), *FDateTime::Now().ToString()));
		auto DebugWriter = IFileManager::Get().CreateFileWriter(*FileName);
		FDiagnosticTableWriterCSV StatWriter(DebugWriter);
		const TSparseArray<ShaderCompilerStats>& PlatformStats = GetShaderCompilerStats();

		StatWriter.AddColumn(TEXT("Path"));
		StatWriter.AddColumn(TEXT("Platform"));
		StatWriter.AddColumn(TEXT("Compiled"));
		StatWriter.AddColumn(TEXT("Cooked"));
		StatWriter.AddColumn(TEXT("Permutations"));
		StatWriter.AddColumn(TEXT("Compiletime"));
		StatWriter.AddColumn(TEXT("CompiledDouble"));
		StatWriter.AddColumn(TEXT("CookedDouble"));
		StatWriter.CycleRow();

		
		for(int32 Platform = 0; Platform < PlatformStats.GetMaxIndex(); ++Platform)
		{
			if(PlatformStats.IsValidIndex(Platform))
			{
				const ShaderCompilerStats& Stats = PlatformStats[Platform];
				for (const auto& Pair : Stats)
				{
					const FString& Path = Pair.Key;
					const FShaderStats& SingleStats = Pair.Value;

					StatWriter.AddColumn(*Path);
					StatWriter.AddColumn(TEXT("%u"), Platform);
					StatWriter.AddColumn(TEXT("%u"), SingleStats.Compiled);
					StatWriter.AddColumn(TEXT("%u"), SingleStats.Cooked);
					StatWriter.AddColumn(TEXT("%u"), SingleStats.PermutationCompilations.Num());
					StatWriter.AddColumn(TEXT("%f"), SingleStats.CompileTime);
					StatWriter.AddColumn(TEXT("%u"), SingleStats.CompiledDouble);
					StatWriter.AddColumn(TEXT("%u"), SingleStats.CookedDouble);
					StatWriter.CycleRow();
					if(GLogShaderCompilerStats)
					{
						UE_LOGF(LogShaderCompilers, Log, "SHADERSTATS %ls, %u, %u, %u, %u, %u, %u\n", *Path, Platform, SingleStats.Compiled, SingleStats.Cooked, SingleStats.PermutationCompilations.Num(), SingleStats.CompiledDouble, SingleStats.CookedDouble);
					}
				}
			}
		}
		DebugWriter->Close();

		FString FullFileName = FPaths::ConvertRelativePathToFull(FileName);
		if (Ar)
		{
			Ar->Logf(DebugText, *FullFileName);
		}
		else
		{
			UE_LOG(LogShaderCompilers, Log, DebugText, *FullFileName);
		}

		if (FParse::Param(FCommandLine::Get(), TEXT("mirrorshaderstats")))
		{
			FString MirrorLocation;
			GConfig->GetString(TEXT("/Script/Engine.ShaderCompilerStats"), TEXT("MaterialStatsLocation"), MirrorLocation, GGameIni);
			FParse::Value(FCommandLine::Get(), TEXT("MaterialStatsMirror="), MirrorLocation);

			if (!MirrorLocation.IsEmpty())
			{
				FString TargetType = "Default";
				FParse::Value(FCommandLine::Get(), TEXT("target="), TargetType);
				if (TargetType == TEXT("Default"))
				{
					FParse::Value(FCommandLine::Get(), TEXT("targetplatform="), TargetType);
				}
				FString CopyLocation = FPaths::Combine(*MirrorLocation, FApp::GetProjectName(), *FApp::GetBranchName(), FString::Printf(TEXT("Stats-Latest-%d(%s).csv"), FEngineVersion::Current().GetChangelist() , *TargetType));
				TArray <FString> ExistingFiles;
				IFileManager::Get().FindFiles(ExistingFiles, *FPaths::Combine(*MirrorLocation, FApp::GetProjectName(), *FApp::GetBranchName()));
				for (FString CurFile : ExistingFiles)
				{
					if (CurFile.Contains(FString::Printf(TEXT("(%s)"), *TargetType)))
					{
						IFileManager::Get().Delete(*FPaths::Combine(*MirrorLocation, FApp::GetProjectName(), *FApp::GetBranchName(), *CurFile), false, true);
					}
				}
				IFileManager::Get().Copy(*CopyLocation, *FileName, true, true);
			}
		}
	}
#endif // ALLOW_DEBUG_FILES
}

template <typename T>
static FString FormatNumber(T Number)
{
	static const FNumberFormattingOptions FormattingOptions = FNumberFormattingOptions().SetUseGrouping(true);
	return FText::AsNumber(Number, &FormattingOptions).ToString();
}

static FString PrintJobsCompletedPercentageToString(int64 JobsAssigned, int64 JobsCompleted)
{
	if (JobsAssigned == 0)
	{
		return TEXT("0%");
	}
	if (JobsAssigned == JobsCompleted)
	{
		return TEXT("100%");
	}

	// With more than a million compile jobs but only a small number that didn't complete,
	// the output might be rounded up to 100%. To avoid a misleading output, we clamp this value to 99.99%
	double JobsCompletedPercentage = 100.0 * (double)JobsCompleted / (double)JobsAssigned;
	return FString::Printf(TEXT("%.2f%%"), FMath::Min(JobsCompletedPercentage, 99.99));
}

static void DumpShaderTimingsToCSVFile(const TCHAR* Filename, const TMap<FString, FShaderTimings>& ShaderTimings)
{
#if ALLOW_DEBUG_FILES
	// Ensure output folder exists
	const FString& OutputDirectory = GShaderCompilingManager->GetAbsoluteShaderDebugInfoDirectory();
	if (!IFileManager::Get().DirectoryExists(*OutputDirectory))
	{
		IFileManager::Get().MakeDirectory(*OutputDirectory, true);
	}

	// Write CSV table to file
	const FString OutputFilename = FPaths::Combine(OutputDirectory, Filename);
	if (TUniquePtr<FArchive> OutputFile = TUniquePtr<FArchive>(IFileManager::Get().CreateFileWriter(*OutputFilename)))
	{
		FDiagnosticTableWriterCSV Table(OutputFile.Get());

		Table.AddColumn(TEXT("SHADER"));
		Table.AddColumn(TEXT("NUMBER OF COMPILATIONS"));
		Table.AddColumn(TEXT("TOTAL COMPILE TIME (s)"));
		Table.AddColumn(TEXT("TOTAL PREPROCESS TIME (s)"));
		Table.AddColumn(TEXT("AVERAGE TIME (s)"));
		Table.AddColumn(TEXT("MAX TIME (s)"));
		Table.AddColumn(TEXT("MIN TIME (s)"));
		Table.CycleRow();

		for (TMap<FString, FShaderTimings>::TConstIterator Iter(ShaderTimings); Iter; ++Iter)
		{
			const FShaderTimings& Timings = Iter.Value();
			Table.AddColumn(TEXT("%s"), *Iter.Key());
			Table.AddColumn(TEXT("%d"), Timings.NumCompiled);
			Table.AddColumn(TEXT("%.2f"), Timings.TotalCompileTime);
			Table.AddColumn(TEXT("%.2f"), Timings.TotalPreprocessTime);
			Table.AddColumn(TEXT("%.2f"), Timings.AverageCompileTime);
			Table.AddColumn(TEXT("%.2f"), Timings.MaxCompileTime);
			Table.AddColumn(TEXT("%.2f"), Timings.MinCompileTime);
			Table.CycleRow();
		}
	}
#endif // ALLOW_DEBUG_FILES
}

static void DumpShaderWorkerDiagnosticsToCSVFile(const TCHAR* Filename, const TArray<FShaderCompilerStats::FWorkerDiagnosticsInfo>& InWorkerDiagnostics)
{
#if ALLOW_DEBUG_FILES
	// Ensure output folder exists
	const FString& OutputDirectory = GShaderCompilingManager->GetAbsoluteShaderDebugInfoDirectory();
	if (!IFileManager::Get().DirectoryExists(*OutputDirectory))
	{
		IFileManager::Get().MakeDirectory(*OutputDirectory, true);
	}

	// Write CSV table to file
	const FString OutputFilename = FPaths::Combine(OutputDirectory, Filename);
	if (TUniquePtr<FArchive> OutputFile = TUniquePtr<FArchive>(IFileManager::Get().CreateFileWriter(*OutputFilename)))
	{
		FDiagnosticTableWriterCSV Table(OutputFile.Get());

		Table.AddColumn(TEXT("BATCH LABEL"));
		Table.AddColumn(TEXT("BATCH INDEX"));
		Table.AddColumn(TEXT("BATCH SIZE"));
		Table.AddColumn(TEXT("WORKER ID"));
		Table.AddColumn(TEXT("MAIN TIMESTAMP"));
		Table.AddColumn(TEXT("PREPARATION (s)"));
		Table.AddColumn(TEXT("DURATION (s)"));
		Table.CycleRow();

		for (const FShaderCompilerStats::FWorkerDiagnosticsInfo& Info : InWorkerDiagnostics)
		{
			Table.AddColumn(TEXT("%s"), *Info.BatchLabel);
			Table.AddColumn(TEXT("%d"), Info.WorkerDiagnosticsOutput.BatchIndex);
			Table.AddColumn(TEXT("%d"), Info.BatchSize);
			if (Info.WorkerId == 0)
			{
				Table.AddColumn(TEXT("n/a"));
			}
			else
			{
				Table.AddColumn(TEXT("%u"), Info.WorkerId);
			}
			Table.AddColumn(TEXT("%.2f"), Info.WorkerDiagnosticsOutput.EntryPointTimestamp);
			Table.AddColumn(TEXT("%.2f"), Info.WorkerDiagnosticsOutput.BatchPreparationTime);
			Table.AddColumn(TEXT("%.2f"), Info.WorkerDiagnosticsOutput.BatchProcessTime);
			Table.CycleRow();
		}
	}
#endif // ALLOW_DEBUG_FILES
}

void FShaderCompilerStats::WriteStatSummary()
{
	const uint32 TotalCompiled = GetTotalShadersCompiled();
	if (TotalCompiled == 0)
	{
		// early out if we haven't done anything yet
		return;
	}

	UE_LOGF(LogShaderCompilers, Display, "================================================");

	const TCHAR* AggregatedSuffix = bMultiProcessAggregated ? TEXT(" (aggregated across all cook processes)") : TEXT("");

	// Only log cache stats if the cache has been queried at least once (this will always be 0 if the job cache is disabled)
	if (Counters.TotalCacheSearchAttempts > 0)
	{
		UE_LOGF(LogShaderCompilers, Display, "=== FShaderJobCache stats%ls ===", AggregatedSuffix);
		UE_LOGF(LogShaderCompilers, Display, "Total job queries %ls, among them cache hits %ls (%.2f%%), DDC hits %ls (%.2f%%), Duplicates %ls (%.2f%%)",
			*FormatNumber(Counters.TotalCacheSearchAttempts),
			*FormatNumber(Counters.TotalCacheHits),
			100.0 * static_cast<double>(Counters.TotalCacheHits) / static_cast<double>(Counters.TotalCacheSearchAttempts),
			*FormatNumber(Counters.TotalCacheDDCHits),
			100.0 * static_cast<double>(Counters.TotalCacheDDCHits) / static_cast<double>(Counters.TotalCacheSearchAttempts),
			*FormatNumber(Counters.TotalCacheDuplicates),
			100.0 * static_cast<double>(Counters.TotalCacheDuplicates) / static_cast<double>(Counters.TotalCacheSearchAttempts));

		UE_LOGF(LogShaderCompilers, Display, "Tracking %ls distinct input hashes that result in %ls distinct outputs (%.2f%%)",
			*FormatNumber(Counters.UniqueCacheInputHashes),
			*FormatNumber(Counters.UniqueCacheOutputs),
			(Counters.UniqueCacheInputHashes > 0) ? 100.0 * static_cast<double>(Counters.UniqueCacheOutputs) / static_cast<double>(Counters.UniqueCacheInputHashes) : 0.0);

		static const FNumberFormattingOptions SizeFormattingOptions = FNumberFormattingOptions().SetMinimumFractionalDigits(2).SetMaximumFractionalDigits(2);

		if (Counters.CacheMemBudget > 0)
		{
			UE_LOGF(LogShaderCompilers, Display, "RAM used: %ls of %ls budget. Usage: %.2f%%",
				*FText::AsMemory(Counters.CacheMemUsed, &SizeFormattingOptions, nullptr, EMemoryUnitStandard::IEC).ToString(),
				*FText::AsMemory(Counters.CacheMemBudget, &SizeFormattingOptions, nullptr, EMemoryUnitStandard::IEC).ToString(),
				100.0 * Counters.CacheMemUsed / Counters.CacheMemBudget);
		}
		else
		{
			UE_LOGF(LogShaderCompilers, Display, "RAM used: %ls, no memory limit set", *FText::AsMemory(Counters.CacheMemUsed, &SizeFormattingOptions, nullptr, EMemoryUnitStandard::IEC).ToString());
		}
	}

	const double TotalTimeAtLeastOneJobWasInFlight = GetTimeShaderCompilationWasActive();

	UE_LOGF(LogShaderCompilers, Display, "=== Shader Compilation stats%ls ===", AggregatedSuffix);
	UE_LOGF(LogShaderCompilers, Display, "Shaders Compiled: %ls", *FormatNumber(TotalCompiled));

	FScopeLock Lock(&CompileStatsLock);	// make a local copy for all the stats?
	UE_LOGF(LogShaderCompilers, Display, "Jobs assigned %ls, completed %ls (%ls)", 
		*FormatNumber(Counters.JobsAssigned),
		*FormatNumber(Counters.JobsCompleted),
		*PrintJobsCompletedPercentageToString(Counters.JobsAssigned, Counters.JobsCompleted));

	if (Counters.LargestViolationMemorySizeInBytes > 0)
	{
		UE_LOGF(LogShaderCompilers, Display,
			"Shader compilation violated current memory limit by %d MiB (%lld bytes). Total worker count %d.",
			static_cast<int32>(Counters.LargestViolationMemorySizeInBytes / 1024 / 1024),
			Counters.LargestViolationMemorySizeInBytes,
			Counters.LargestViolationWorkerCount);
	}

	if (Counters.TimesLocalWorkersWereIdle > 0.0)
	{
		UE_LOGF(LogShaderCompilers, Display, "Average time worker was idle: %.2f s", Counters.AccumulatedLocalWorkerIdleTime / Counters.TimesLocalWorkersWereIdle);
	}

	if (Counters.JobsAssigned > 0)
	{
		UE_LOGF(LogShaderCompilers, Display, "Time job spent in pending queue: average %.2f s, longest %.2f s", Counters.AccumulatedPendingTime / (double)Counters.JobsAssigned, Counters.MaxPendingTime);
	}

	if (Counters.JobsCompleted > 0)
	{
		UE_LOGF(LogShaderCompilers, Display, "Job execution time: average %.2f s, max %.2f s", Counters.AccumulatedJobExecutionTime / (double)Counters.JobsCompleted, Counters.MaxJobExecutionTime);
		UE_LOGF(LogShaderCompilers, Display, "Job life time (pending + execution): average %.2f s, max %.2f", Counters.AccumulatedJobLifeTime / (double)Counters.JobsCompleted, Counters.MaxJobLifeTime);
	}

	if (Counters.NumAccumulatedShaderCodes > 0)
	{
		const FString TotalCodeSizeStr = FText::AsMemory(Counters.AccumulatedShaderCodeSize).ToString();
		const FString NumShadersStr = FormatNumber(Counters.NumAccumulatedShaderCodes);
		const FString AvgCodeSizeStr = FText::AsMemory((uint64)((double)Counters.AccumulatedShaderCodeSize / (double)Counters.NumAccumulatedShaderCodes)).ToString();
		const FString MinCodeSizeStr = FText::AsMemory((uint64)Counters.MinShaderCodeSize).ToString();
		const FString MaxCodeSizeStr = FText::AsMemory((uint64)Counters.MaxShaderCodeSize).ToString();
		UE_LOGF(LogShaderCompilers, Display, "Shader code size: total %ls, numShaders %ls, average %ls, min %ls, max %ls", *TotalCodeSizeStr, *NumShadersStr, *AvgCodeSizeStr, *MinCodeSizeStr, *MaxCodeSizeStr);
	}

	UE_LOGF(LogShaderCompilers, Display, "Time at least one job was in flight (either pending or executed): %.2f s", TotalTimeAtLeastOneJobWasInFlight);

	// print stats about the batches
	if (Counters.LocalJobBatchesSeen > 0 && Counters.DistributedJobBatchesSeen > 0)
	{
		int64 JobBatchesSeen = Counters.LocalJobBatchesSeen + Counters.DistributedJobBatchesSeen;
		double TotalJobsReportedInJobBatches = Counters.TotalJobsReportedInLocalJobBatches + Counters.TotalJobsReportedInDistributedJobBatches;

		UE_LOGF(LogShaderCompilers, Display, "Jobs were issued in %ls batches (%ls local, %ls distributed), average %.2f jobs/batch (%.2f jobs/local batch. %.2f jobs/distributed batch)",
			*FormatNumber(JobBatchesSeen), *FormatNumber(Counters.LocalJobBatchesSeen), *FormatNumber(Counters.DistributedJobBatchesSeen),
			static_cast<double>(TotalJobsReportedInJobBatches) / static_cast<double>(JobBatchesSeen),
			static_cast<double>(Counters.TotalJobsReportedInLocalJobBatches) / static_cast<double>(Counters.LocalJobBatchesSeen),
			static_cast<double>(Counters.TotalJobsReportedInDistributedJobBatches) / static_cast<double>(Counters.DistributedJobBatchesSeen)
		);
	}
	else if (Counters.LocalJobBatchesSeen > 0)
	{
		UE_LOGF(LogShaderCompilers, Display, "Jobs were issued in %ls batches (only local compilation was used), average %.2f jobs/batch", 
			*FormatNumber(Counters.LocalJobBatchesSeen), static_cast<double>(Counters.TotalJobsReportedInLocalJobBatches) / static_cast<double>(Counters.LocalJobBatchesSeen));
	}
	else if (Counters.DistributedJobBatchesSeen > 0)
	{
		UE_LOGF(LogShaderCompilers, Display, "Jobs were issued in %ls batches (only distributed compilation was used), average %.2f jobs/batch",
			*FormatNumber(Counters.DistributedJobBatchesSeen), static_cast<double>(Counters.TotalJobsReportedInDistributedJobBatches) / static_cast<double>(Counters.DistributedJobBatchesSeen));
	}

	if (TotalTimeAtLeastOneJobWasInFlight > 0.0)
	{
		UE_LOGF(LogShaderCompilers, Display, "Average processing rate: %.2f jobs/sec", (double)Counters.JobsCompleted / TotalTimeAtLeastOneJobWasInFlight);
	}

	if (ShaderTimings.Num())
	{
		// calculate effective parallelization (total time needed to compile all shaders divided by actual wall clock time spent processing at least 1 shader)
		double TotalThreadTimeForAllShaders = 0.0;
		double TotalThreadPreprocessTimeForAllShaders = 0.0;
		for (TMap<FString, FShaderTimings>::TConstIterator Iter(ShaderTimings); Iter; ++Iter)
		{
			TotalThreadTimeForAllShaders += Iter.Value().TotalCompileTime;
			TotalThreadPreprocessTimeForAllShaders += Iter.Value().TotalPreprocessTime;
		}

		UE_LOGF(LogShaderCompilers, Display, "Total thread time: %ls s", *FormatNumber(TotalThreadTimeForAllShaders));
		UE_LOGF(LogShaderCompilers, Display, "Total thread preprocess time: %ls s", *FormatNumber(TotalThreadPreprocessTimeForAllShaders));
		UE_LOGF(LogShaderCompilers, Display, "Percentage time preprocessing: %.2f%%", TotalThreadTimeForAllShaders > 0.0 ? (TotalThreadPreprocessTimeForAllShaders / TotalThreadTimeForAllShaders) * 100.0 : 0.0);

		if (Counters.MaxRemoteAgents > 0)
		{
			UE_LOGF(LogShaderCompilers, Display, "Highest number of remote agents active in parallel: %u (%u active cores peak)", Counters.MaxRemoteAgents, Counters.MaxActiveAgentCores);
		}

		if (TotalTimeAtLeastOneJobWasInFlight > 0.0)
		{
			double EffectiveParallelization = TotalThreadTimeForAllShaders / TotalTimeAtLeastOneJobWasInFlight;
			if (Counters.DistributedJobBatchesSeen == 0)
			{
				UE_LOGF(LogShaderCompilers, Display, "Effective parallelization: %.2f (times faster than compiling all shaders on one thread). Compare with number of workers: %d - %f", EffectiveParallelization, GShaderCompilingManager->GetNumLocalWorkers(), EffectiveParallelization/(double)GShaderCompilingManager->GetNumLocalWorkers());
			}
			else
			{
				UE_LOGF(LogShaderCompilers, Display, "Effective parallelization: %.2f (times faster than compiling all shaders on one thread). Distributed compilation was used.", EffectiveParallelization);
			}
		}


		// sort by avg time
		ShaderTimings.ValueSort([](const FShaderTimings& A, const FShaderTimings& B) { return A.AverageCompileTime > B.AverageCompileTime; });

		const int32 MaxShadersToPrint = FMath::Min(ShaderTimings.Num(), GMaxShaderStatsToLog);
		UE_LOGF(LogShaderCompilers, Display, "Top %d most expensive shader types by average time:", MaxShadersToPrint);

		int32 Idx = 0;
		for (TMap<FString, FShaderTimings>::TConstIterator Iter(ShaderTimings); Iter; ++Iter)
		{
			const FShaderTimings& Timings = Iter.Value();

			UE_LOGF(LogShaderCompilers, Display, "%60ls (compiled %4d times, average %4.2f sec, max %4.2f sec, min %4.2f sec)", *Iter.Key(), Timings.NumCompiled, Timings.AverageCompileTime, Timings.MaxCompileTime, Timings.MinCompileTime);
			if (++Idx >= MaxShadersToPrint)
			{
				break;
			}
		}

		if (GDumpShaderTimeStats)
		{
			DumpShaderTimingsToCSVFile(TEXT("ShaderTimings.SortedByAverageTime.csv"), ShaderTimings);
		}

		// sort by total time
		ShaderTimings.ValueSort([](const FShaderTimings& A, const FShaderTimings& B) { return A.TotalCompileTime > B.TotalCompileTime; });

		UE_LOGF(LogShaderCompilers, Display, "Top %d shader types by total compile time:", MaxShadersToPrint);

		Idx = 0;
		for (TMap<FString, FShaderTimings>::TConstIterator Iter(ShaderTimings); Iter; ++Iter)
		{
			const FShaderTimings& Timings = Iter.Value();

			UE_LOGF(LogShaderCompilers, Display, "%60ls - %.2f%% of total time (compiled %4d times, average %4.2f sec, max %4.2f sec, min %4.2f sec)", 
				*Iter.Key(), 100.0 * Timings.TotalCompileTime / TotalThreadTimeForAllShaders, Timings.NumCompiled, Timings.AverageCompileTime, Timings.MaxCompileTime, Timings.MinCompileTime);
			if (++Idx >= MaxShadersToPrint)
			{
				break;
			}
		}

		if (GDumpShaderTimeStats)
		{
			DumpShaderTimingsToCSVFile(TEXT("ShaderTimings.SortedByTotalTime.csv"), ShaderTimings);
		}
	}

	if (WorkerDiagnostics.Num())
	{
		if (GShaderCompilerDumpWorkerDiagnostics)
		{
			DumpShaderWorkerDiagnosticsToCSVFile(TEXT("ShaderCompileWorker.Diagnostics.csv"), WorkerDiagnostics);
		}
	}

	MaterialCounters.WriteStatSummary(AggregatedSuffix);

	UE_LOGF(LogShaderCompilers, Display, "================================================");
}

void FShaderCompilerStats::GatherAnalytics(const FString& BaseName, TArray<FAnalyticsEventAttribute>& Attributes)
{
	const double TotalTimeAtLeastOneJobWasInFlight = GetTimeShaderCompilationWasActive();

	FScopeLock Lock(&CompileStatsLock);

	{
		FString AttrName = BaseName + TEXT("ShadersCompiled");
		Attributes.Emplace(MoveTemp(AttrName), Counters.JobsCompleted);
	}

	if (ShaderTimings.Num())
	{
		double TotalThreadTimeForAllShaders = 0.0;
		double TotalThreadPreprocessTimeForAllShaders = 0.0;
		for (TMap<FString, FShaderTimings>::TConstIterator Iter(ShaderTimings); Iter; ++Iter)
		{
			TotalThreadTimeForAllShaders += Iter.Value().TotalCompileTime;
			TotalThreadPreprocessTimeForAllShaders += Iter.Value().TotalPreprocessTime;
		}

		{
			FString AttrName = BaseName + TEXT("TotalThreadTime");
			Attributes.Emplace(MoveTemp(AttrName), TotalThreadTimeForAllShaders);
		}

		{
			FString AttrName = BaseName + TEXT("TotalThreadPreprocessTime");
			Attributes.Emplace(MoveTemp(AttrName), TotalThreadPreprocessTimeForAllShaders);
		}

		{
            const double EffectiveParallelization = TotalTimeAtLeastOneJobWasInFlight > 0.0 ? TotalThreadTimeForAllShaders / TotalTimeAtLeastOneJobWasInFlight : 0.0;
			FString AttrName = BaseName + TEXT("EffectiveParallelization");
			Attributes.Emplace(MoveTemp(AttrName), EffectiveParallelization);
		}
	}

	if (Counters.TotalCacheSearchAttempts)
	{
		const FString ChildName = "JobCache_";

		{
			FString AttrName = BaseName + ChildName + TEXT("Queries");
			Attributes.Emplace(MoveTemp(AttrName), Counters.TotalCacheSearchAttempts);
		}

		{
			FString AttrName = BaseName + ChildName + TEXT("Hits");
			Attributes.Emplace(MoveTemp(AttrName), Counters.TotalCacheHits);
		}

		{
			FString AttrName = BaseName + ChildName + TEXT("DDCHits");
			Attributes.Emplace(MoveTemp(AttrName), Counters.TotalCacheDDCHits);
		}

		{
			FString AttrName = BaseName + ChildName + TEXT("NumInputs");
			Attributes.Emplace(MoveTemp(AttrName), Counters.UniqueCacheInputHashes);
		}

		{
			FString AttrName = BaseName + ChildName + TEXT("NumOutputs");
			Attributes.Emplace(MoveTemp(AttrName), Counters.UniqueCacheOutputs);
		}

		{
			FString AttrName = BaseName + ChildName + TEXT("MemUsed");
			Attributes.Emplace(MoveTemp(AttrName), Counters.CacheMemUsed);
		}

		{
			FString AttrName = BaseName + ChildName + TEXT("MemBudget");
			Attributes.Emplace(MoveTemp(AttrName), Counters.CacheMemBudget);
		}
	}

	MaterialCounters.GatherAnalytics(Attributes);
}

uint32 FShaderCompilerStats::GetTotalShadersCompiled()
{
	FScopeLock Lock(&CompileStatsLock);
	return (uint32)FMath::Max(0ll, Counters.JobsCompleted);
}

static void AddToInterval(TArray<TInterval<double>>& Accumulator, const TInterval<double>& NewInterval)
{
	bool bFoundOverlap = false;
	TInterval<double> New = NewInterval;
	int32 Idx = 0;
	do
	{
		bFoundOverlap = false;
		for (; Idx < Accumulator.Num(); ++Idx)
		{
			const TInterval<double>& Existing = Accumulator[Idx];
			if (Existing.Max < New.Min)
			{
				continue;	// no overlap but the new interval starts after this one ends, keep searching
			}

			if (New.Max < Existing.Min)
			{
				break;		// no overlap, but the new interval ends before this one starts, insert here
			}

			// if fully contained within existing interval, just ignore
			if (Existing.Min <= New.Min && New.Max <= Existing.Max)
			{
				return;
			}

			bFoundOverlap = true;
			// if there's an overlap, remove the existing interval, merge with the new one and attempt to add again
			TInterval<double> Merged(FMath::Min(Existing.Min, New.Min), FMath::Max(Existing.Max, New.Max));
			check(Merged.Size() >= Existing.Size());
			check(Merged.Size() >= New.Size());
			Accumulator.RemoveAt(Idx);
			New = Merged;
			break;
		}
	} while (bFoundOverlap);

	// if we arrived here without an overlap, we have a new one; insert in the appropriate place
	if (!bFoundOverlap)
	{
		Accumulator.Insert(New, Idx);
	}
}

void FShaderCompilerStats::Aggregate(FShaderCompilerStats& Other)
{
	// note: intentionally not taking local lock as this should only ever be called on a local copy of the stats object
	FScopeLock Lock(&Other.CompileStatsLock);
	Counters += Other.Counters;

	for (TSparseArray<ShaderCompilerStats>::TConstIterator It(Other.CompileStats); It; ++It)
	{
		if (!CompileStats.IsValidIndex(It.GetIndex()))
		{
			CompileStats.EmplaceAt(It.GetIndex());
		}

		ShaderCompilerStats& Stats = CompileStats[It.GetIndex()];
		for (const TPair<FString, FShaderStats>& StatsKeyValue : *It)
		{
			FShaderStats* Current = Stats.Find(StatsKeyValue.Key);
			if (Current)
			{
				*Current += StatsKeyValue.Value;
			}
			else
			{
				Stats.Add(StatsKeyValue);
			}
		}
	}

	// note: this is suboptimal (O(n^2)) but there aren't a lot of these in practice
	for (const TInterval<double>& Interval : Other.JobLifeTimeIntervals)
	{
		AddToInterval(JobLifeTimeIntervals, Interval);
	}

	for (const TPair<FString, FShaderTimings>& TimingsKeyValue : Other.ShaderTimings)
	{
		FShaderTimings* Current = ShaderTimings.Find(TimingsKeyValue.Key);
		if (Current)
		{
			*Current += TimingsKeyValue.Value;
		}
		else
		{
			ShaderTimings.Add(TimingsKeyValue);
		}
	}

	MaterialCounters += Other.MaterialCounters;

	if (GShaderCompilerDumpWorkerDiagnostics)
	{
		WorkerDiagnostics.Append(Other.WorkerDiagnostics);
	}
}


TSharedPtr<FJsonObject> FShaderCompilerStats::ToJson()
{
	TSharedPtr<FJsonObject> RootObject(MakeShared<FJsonObject>());
	{
		TSharedRef<FJsonObject> JsonObject = MakeShared<FJsonObject>();
		FJsonObjectConverter::UStructToJsonObject(FShaderCompilerCounters::StaticStruct(), &Counters, JsonObject, 0, 0, nullptr, EJsonObjectConversionFlags::SkipStandardizeCase);
		RootObject->SetObjectField(TEXT("Counters"), JsonObject);
	}
	{
		TSharedRef<FJsonObject> JsonObject = MakeShared<FJsonObject>();
		FJsonObjectConverter::UStructToJsonObject(FShaderCompilerMaterialCounters::StaticStruct(), &MaterialCounters, JsonObject, 0, 0, nullptr, EJsonObjectConversionFlags::SkipStandardizeCase);
		RootObject->SetObjectField(TEXT("MaterialCounters"), JsonObject);
	}

	TArray<TSharedPtr<FJsonValue>> CompileStatsArray;
	for (TSparseArray<ShaderCompilerStats>::TConstIterator It(CompileStats); It; ++It)
	{
		if (!CompileStats.IsValidIndex(It.GetIndex()))
		{
			continue;
		}

		TSharedPtr<FJsonObject> InnerObject(MakeShared<FJsonObject>());
		for (const TPair<FString, FShaderStats>& Pair : *It)
		{
			TSharedRef<FJsonObject> JsonObject = MakeShared<FJsonObject>();
			FJsonObjectConverter::UStructToJsonObject(FShaderStats::StaticStruct(), &Pair.Value, JsonObject, 0, 0, nullptr, EJsonObjectConversionFlags::SkipStandardizeCase);
			InnerObject->SetObjectField(Pair.Key, JsonObject);
		}
		CompileStatsArray.Push(MakeShared<FJsonValueObject>(InnerObject));
	}
	RootObject->SetArrayField(TEXT("CompileStats"), CompileStatsArray);

	TSharedPtr<FJsonObject> ShaderTimingsObject(MakeShared<FJsonObject>());
	for (const TPair<FString, FShaderTimings>& TimingPair : ShaderTimings)
	{
		TSharedRef<FJsonObject> JsonObject = MakeShared<FJsonObject>();
		FJsonObjectConverter::UStructToJsonObject(FShaderTimings::StaticStruct(), &TimingPair.Value, JsonObject, 0, 0, nullptr, EJsonObjectConversionFlags::SkipStandardizeCase);
		ShaderTimingsObject->SetObjectField(TimingPair.Key, JsonObject);
	}
	RootObject->SetObjectField(TEXT("ShaderTimings"), ShaderTimingsObject);
	return RootObject;
}

void FShaderCompilerStats::WriteToCompactBinary(FCbWriter& Writer)
{
	FScopeLock Lock(&CompileStatsLock);
	Writer.AddBinary("Counters", &Counters, sizeof(Counters));

	Writer.AddBinary("MaterialCounters", &MaterialCounters, sizeof(MaterialCounters));

	Writer.BeginArray("CompileStatIndices");	
	// Write the array of valid indices this worker has in the compile stats sparse array
	for (TSparseArray<ShaderCompilerStats>::TConstIterator It(CompileStats); It; ++It)
	{
		if (CompileStats.IsValidIndex(It.GetIndex()))
		{
			Writer << It.GetIndex();
		}
	}
	Writer.EndArray();

	Writer.BeginArray("CompileStats");
		// Then write the actual compile stats maps in the same order as the above indices
	for (TSparseArray<ShaderCompilerStats>::TConstIterator It(CompileStats); It; ++It)
	{
		if (!CompileStats.IsValidIndex(It.GetIndex()))
		{
			continue;
		}

		Writer.BeginObject();
		Writer.BeginArray("CompileStatsKeys");
		for (TPair<FString, FShaderStats> Pair : *It)
		{
			Writer << Pair.Key;
		}
		Writer.EndArray();

		Writer.BeginArray("CompileStatsValues");
		for (const TPair<FString, FShaderStats>& Pair : *It)
		{
			Writer.BeginObject();
			Writer << "Compiled" << Pair.Value.Compiled;
			Writer << "CompiledDouble" << Pair.Value.CompiledDouble;
			Writer << "CompileTime" << Pair.Value.CompileTime;
			Writer << "Cooked" << Pair.Value.Cooked;
			Writer << "CookedDouble" << Pair.Value.CookedDouble;
			Writer.BeginArray("PermutationCompilations");
			for (const FShaderCompilerSinglePermutationStat& Stat : Pair.Value.PermutationCompilations)
			{
				Writer.BeginObject();
				Writer << "Compiled" << Stat.Compiled;
				Writer << "CompiledDouble" << Stat.CompiledDouble;
				Writer << "Cooked" << Stat.Cooked;
				Writer << "CookedDouble" << Stat.CookedDouble;
				Writer << "PermutationHash" << Stat.PermutationHash;
				Writer.EndObject();
			}
			Writer.EndArray();
			Writer.EndObject();
		}
		Writer.EndArray();
		Writer.EndObject();
	}
	Writer.EndArray();

	Writer.BeginArray("JobLifeTimeIntervals");
	for (const TInterval<double>& Interval : JobLifeTimeIntervals)
	{
		Writer.AddBinary(&Interval, sizeof(TInterval<double>));
	}
	Writer.EndArray();

	Writer.BeginArray("ShaderTimingsKeys");
	for (const TPair<FString, FShaderTimings>& TimingPair : ShaderTimings)
	{
		Writer << TimingPair.Key;
	}
	Writer.EndArray();

	Writer.BeginArray("ShaderTimingsValues");
	for (const TPair<FString, FShaderTimings>& TimingPair : ShaderTimings)
	{
		Writer.AddBinary(&TimingPair.Value, sizeof(FShaderTimings));
	}
	Writer.EndArray();
}

void FShaderCompilerStats::ReadFromCompactBinary(FCbObjectView& Reader)
{
	FScopeLock Lock(&CompileStatsLock);
	FMemoryView CountersMem = Reader["Counters"].AsBinaryView();
	check(CountersMem.GetSize() == sizeof(FShaderCompilerCounters));
	Counters = *reinterpret_cast<const FShaderCompilerCounters*>(CountersMem.GetData());

	FMemoryView MaterialCountersMem = Reader["MaterialCounters"].AsBinaryView();
	check(MaterialCountersMem.GetSize() == sizeof(FShaderCompilerMaterialCounters));
	MaterialCounters = *reinterpret_cast<const FShaderCompilerMaterialCounters*>(MaterialCountersMem.GetData());

	FCbArrayView CompileStatIndicesView = Reader["CompileStatIndices"].AsArrayView();
	FCbArrayView CompileStatsView = Reader["CompileStats"].AsArrayView();
	check(CompileStatIndicesView.Num() == CompileStatsView.Num());

	FCbFieldViewIterator IndexIt = CompileStatIndicesView.CreateViewIterator();
	FCbFieldViewIterator StatsIt = CompileStatsView.CreateViewIterator();

	while (IndexIt && StatsIt)
	{
		if (!CompileStats.IsValidIndex(IndexIt.AsUInt32()))
		{
			FSparseArrayAllocationInfo AllocInfo = CompileStats.InsertUninitialized(IndexIt.AsUInt32());
			new(AllocInfo) ShaderCompilerStats();
		}
		ShaderCompilerStats& Stats = CompileStats[IndexIt.AsUInt32()];

		FCbObjectView PlatformStatsObject = StatsIt->AsObjectView();
		FCbArrayView StatsKeysView = PlatformStatsObject["CompileStatsKeys"].AsArrayView();
		FCbArrayView StatsValuesView = PlatformStatsObject["CompileStatsValues"].AsArrayView();
		check(StatsKeysView.Num() == StatsValuesView.Num());

		Stats.Reserve(StatsKeysView.Num());

		FCbFieldViewIterator KeysIt = StatsKeysView.CreateViewIterator();
		FCbFieldViewIterator ValuesIt = StatsValuesView.CreateViewIterator();

		while (KeysIt && ValuesIt)
		{
			FCbObjectView ShaderStatsObject = ValuesIt->AsObjectView();
			FShaderStats& ShaderStats = Stats.Add(FString(KeysIt->AsString()));
			ShaderStats.Compiled = ShaderStatsObject["Compiled"].AsUInt32();
			ShaderStats.CompiledDouble = ShaderStatsObject["CompiledDouble"].AsUInt32();
			ShaderStats.CompileTime = ShaderStatsObject["CompileTime"].AsFloat();
			ShaderStats.Cooked = ShaderStatsObject["Cooked"].AsUInt32();
			ShaderStats.CookedDouble = ShaderStatsObject["CookedDouble"].AsUInt32();

			FCbArrayView PermutationsArrayView = ShaderStatsObject["PermutationCompilations"].AsArrayView();
			ShaderStats.PermutationCompilations.Reset(PermutationsArrayView.Num());
			for (FCbFieldView CompilationField : PermutationsArrayView)
			{
				FCbObjectView PermutationObject = CompilationField.AsObjectView();
				uint32 Index = ShaderStats.PermutationCompilations.Emplace
				(
					PermutationObject["PermutationHash"].AsUInt64(),
					PermutationObject["Compiled"].AsUInt32(),
					PermutationObject["Cooked"].AsUInt32()
				);
				ShaderStats.PermutationCompilations[Index].CompiledDouble = PermutationObject["CompiledDouble"].AsUInt32();
				ShaderStats.PermutationCompilations[Index].CookedDouble = PermutationObject["CookedDouble"].AsUInt32();
			}

			++ValuesIt;
			++KeysIt;
		}

		++IndexIt;
		++StatsIt;
	}

	FCbArrayView JobLifeTimeIntervalsView = Reader["JobLifeTimeIntervals"].AsArrayView();
	JobLifeTimeIntervals.Reset(JobLifeTimeIntervalsView.Num());
	for (FCbFieldView JobLifeTimeField : JobLifeTimeIntervalsView)
	{
		FMemoryView IntervalObj = JobLifeTimeField.AsBinaryView();
		check(IntervalObj.GetSize() == sizeof(TInterval<double>));
		JobLifeTimeIntervals.Add(*reinterpret_cast<const TInterval<double>*>(IntervalObj.GetData()));
	}

	FCbArrayView TimingsKeysView = Reader["ShaderTimingsKeys"].AsArrayView();
	FCbArrayView TimingsValuesView = Reader["ShaderTimingsValues"].AsArrayView();
	check(TimingsKeysView.Num() == TimingsValuesView.Num());

	ShaderTimings.Reserve(TimingsKeysView.Num());

	FCbFieldViewIterator TimingsKeysIt = TimingsKeysView.CreateViewIterator();
	FCbFieldViewIterator TimingsValuesIt = TimingsValuesView.CreateViewIterator();

	while (TimingsKeysIt && TimingsValuesIt)
	{
		FMemoryView TimingsValuesBinary = TimingsValuesIt->AsBinaryView();
		check(TimingsValuesBinary.GetSize() == sizeof(FShaderTimings));
		ShaderTimings.Add(FString(TimingsKeysIt->AsString()), *reinterpret_cast<const FShaderTimings*>(TimingsValuesBinary.GetData()));
		++TimingsKeysIt;
		++TimingsValuesIt;
	}
}

void FShaderCompilerStats::RegisterLocalWorkerIdleTime(double IdleTime)
{
	FScopeLock Lock(&CompileStatsLock);
	Counters.AccumulatedLocalWorkerIdleTime += IdleTime;
	Counters.TimesLocalWorkersWereIdle++;
}

TRACE_DECLARE_INT_COUNTER(Shaders_Pending, TEXT("Shaders/Pending"));
void FShaderCompilerStats::RegisterNewPendingJob(FShaderCommonCompileJob& Job)
{
	// accessing job timestamps isn't arbitrated by any lock. It is assumed that the registration of a job at one of the stages
	// of its lifetime happens before the code can move it to another stage (i.e. new pending job is registered before it is added to the pending queue,
	// so it cannot be given away to a worker while it's still being registered, and an assigned job is registered before it is actually given to the worker,
	// so it cannot end up being registered as finished at the same time on some other thread).
	Job.TimeAddedToPendingQueue = FPlatformTime::Seconds();
	TRACE_COUNTER_ADD(Shaders_Pending, 1);
}

void FShaderCompilerStats::RegisterAssignedJob(FShaderCommonCompileJob& Job)
{
	ensure(Job.TimeAddedToPendingQueue != 0.0);
	Job.TimeAssignedToExecution = FPlatformTime::Seconds();

	FScopeLock Lock(&CompileStatsLock);
	Counters.JobsAssigned++;
	double TimeSpendPending = (Job.TimeAssignedToExecution - Job.TimeAddedToPendingQueue);
	Counters.AccumulatedPendingTime += TimeSpendPending;
	Counters.MaxPendingTime = FMath::Max(TimeSpendPending, Counters.MaxPendingTime);
}

TRACE_DECLARE_INT_COUNTER(Shaders_Compiled, TEXT("Shaders/Compiled"));
void FShaderCompilerStats::RegisterFinishedJob(FShaderCommonCompileJob& Job)
{
	FScopeLock Lock(&CompileStatsLock);

	bool bCompilationSkipped = Job.JobStatusPtr->WasCompilationSkipped();
	if (!bCompilationSkipped)
	{
		ensure(Job.TimeAssignedToExecution != 0.0);
		Job.TimeExecutionCompleted = FPlatformTime::Seconds();
		TRACE_COUNTER_ADD(Shaders_Compiled, 1);
		Counters.JobsCompleted++;

		double ExecutionTime = (Job.TimeExecutionCompleted - Job.TimeAssignedToExecution);
		Counters.AccumulatedJobExecutionTime += ExecutionTime;
		Counters.MaxJobExecutionTime = FMath::Max(ExecutionTime, Counters.MaxJobExecutionTime);

		double LifeTime = (Job.TimeExecutionCompleted - Job.TimeAddedToPendingQueue);
		Counters.AccumulatedJobLifeTime += LifeTime;
		Counters.MaxJobLifeTime = FMath::Max(LifeTime, Counters.MaxJobLifeTime);
		
		// estimate lifetime without an overlap
		ensure(Job.TimeAddedToPendingQueue != 0.0 && Job.TimeAddedToPendingQueue <= Job.TimeExecutionCompleted);
		AddToInterval(JobLifeTimeIntervals, TInterval<double>(Job.TimeAddedToPendingQueue, Job.TimeExecutionCompleted));
	}
	
	auto RegisterStatsFromSingleJob = [this, bCompilationSkipped](const FShaderCompileJob& SingleJob)
	{
		// Register min/max/average shader code sizes for single job output
		const int32 ShaderCodeSize = SingleJob.Output.ShaderCode.GetShaderCodeSize();
		if (!bCompilationSkipped && ShaderCodeSize > 0)
		{
			Counters.MinShaderCodeSize = (Counters.MinShaderCodeSize > 0 ? FMath::Min(Counters.MinShaderCodeSize, ShaderCodeSize) : ShaderCodeSize);
			Counters.MaxShaderCodeSize = (Counters.MaxShaderCodeSize > 0 ? FMath::Max(Counters.MaxShaderCodeSize, ShaderCodeSize) : ShaderCodeSize);
			Counters.AccumulatedShaderCodeSize += (uint64)ShaderCodeSize;
			++Counters.NumAccumulatedShaderCodes;
		}

		// Sanity check; compile time should be 0 for cache hits
		check(!bCompilationSkipped || SingleJob.Output.CompileTime == 0.0f);
		// Preprocess time should always be non-zero if preprocessing succeeded; note that preprocessing for pipeline stage jobs may be skipped 
		// in the case preprocessing a preceding stage of the pipeline failed
		check(!SingleJob.PreprocessOutput.GetSucceeded() || SingleJob.Output.PreprocessTime > 0.0f);

		const FString ShaderName(SingleJob.Key.ShaderType->GetName());
		if (FShaderTimings* Existing = ShaderTimings.Find(ShaderName))
		{
			// Always want to log preprocess time, in case preprocessed cache is enabled and preprocessing ran in the cooker prior to compilation
			// (PreprocessTime will be 0 if preprocessed cache is disabled)
			Existing->TotalPreprocessTime += SingleJob.Output.PreprocessTime;
			if (!bCompilationSkipped)
			{
				// If no actual compiles have been logged yet, min compile time is just the compile time of this job (first to actually run)
				Existing->MinCompileTime = Existing->NumCompiled ? FMath::Min(Existing->MinCompileTime, static_cast<float>(SingleJob.Output.CompileTime)) : SingleJob.Output.CompileTime;
				Existing->MaxCompileTime = FMath::Max(Existing->MaxCompileTime, static_cast<float>(SingleJob.Output.CompileTime));
				Existing->TotalCompileTime += SingleJob.Output.CompileTime;
				Existing->NumCompiled++;
				// calculate as an optimization to make sorting later faster
				Existing->AverageCompileTime = Existing->TotalCompileTime / static_cast<float>(Existing->NumCompiled);
			}
		}
		else
		{
			FShaderTimings New;
			New.MinCompileTime = SingleJob.Output.CompileTime;
			New.MaxCompileTime = New.MinCompileTime;
			New.TotalCompileTime = New.MinCompileTime;
			New.AverageCompileTime = New.MinCompileTime;
			// It's possible the first entry for a given shader didn't actually compile (i.e. hit in DDC)
			// so we need to account for that in the stats
			New.NumCompiled = bCompilationSkipped ? 0 : 1;
			New.TotalPreprocessTime += SingleJob.Output.PreprocessTime;

			ShaderTimings.Add(ShaderName, New);
		}
	};

	Job.ForEachSingleShaderJob(RegisterStatsFromSingleJob);
}

void FShaderCompilerStats::RegisterJobBatch(int32 NumJobs, EExecutionType ExecType)
{
	if (ExecType == EExecutionType::Local)
	{
		FScopeLock Lock(&CompileStatsLock);
		++Counters.LocalJobBatchesSeen;
		Counters.TotalJobsReportedInLocalJobBatches += NumJobs;
	}
	else if (ExecType == EExecutionType::Distributed)
	{
		FScopeLock Lock(&CompileStatsLock);
		++Counters.DistributedJobBatchesSeen;
		Counters.TotalJobsReportedInDistributedJobBatches += NumJobs;
	}
	else
	{
		checkNoEntry();
	}
}

void FShaderCompilerStats::RegisterDistributedBuildStats(const FDistributedBuildStats& InStats)
{
	FScopeLock Lock(&CompileStatsLock);
	Counters.MaxRemoteAgents = FMath::Max(Counters.MaxRemoteAgents, InStats.MaxRemoteAgents);
	Counters.MaxActiveAgentCores = FMath::Max(Counters.MaxActiveAgentCores, InStats.MaxActiveAgentCores);
}

void FShaderCompilerStats::RegisterWorkerDiagnostics(const FShaderCompileWorkerDiagnostics& InDiagnostics, FString&& InBatchLabel, int32 InBatchSize, uint32 InWorkerId)
{
	FScopeLock Lock(&CompileStatsLock);
	FWorkerDiagnosticsInfo& NewDiagnosticsInfo = WorkerDiagnostics.AddDefaulted_GetRef();
	NewDiagnosticsInfo.WorkerDiagnosticsOutput = InDiagnostics;
	NewDiagnosticsInfo.BatchLabel = MoveTemp(InBatchLabel);
	NewDiagnosticsInfo.BatchSize = InBatchSize;
	NewDiagnosticsInfo.WorkerId = InWorkerId;
}

void FShaderCompilerStats::RegisterMemoryViolation(const int64 InMemorySizeInBytes, const int32 InWorkerCount)
{
	FScopeLock Lock(&CompileStatsLock);

	if (InMemorySizeInBytes < Counters.LargestViolationMemorySizeInBytes)
		return;

	Counters.LargestViolationMemorySizeInBytes = InMemorySizeInBytes;
	Counters.LargestViolationWorkerCount = InWorkerCount;
}

void FShaderCompilerMaterialCounters::WriteStatSummary(const TCHAR* AggregatedSuffix)
{
	auto CalcTimePercentage = [&](double Val) {
		return  (int)round(Val / FMath::Max(1e-6, MaterialTranslateTotalTimeSec) * 100);
	};

	UE_LOGF(LogShaderCompilers, Display, "=== Material stats%ls ===", AggregatedSuffix);
	UE_LOGF(LogShaderCompilers, Display, "Materials Cooked:        %d", NumMaterialsCooked);
	UE_LOGF(LogShaderCompilers, Display, "Materials Translated:    %d", MaterialTranslateCalls);
	UE_LOGF(LogShaderCompilers, Display, "Material Total Translate Time: %.2f s", MaterialTranslateTotalTimeSec);
	UE_LOGF(LogShaderCompilers, Display, "Material Translation Only: %.2f s (%d%%)", MaterialTranslateTranslationOnlyTimeSec, CalcTimePercentage(MaterialTranslateTranslationOnlyTimeSec));
	UE_LOGF(LogShaderCompilers, Display, "Material DDC Serialization Only: %.2f s (%d%%)", MaterialTranslateSerializationOnlyTimeSec, CalcTimePercentage(MaterialTranslateSerializationOnlyTimeSec));

	int HitsPercentage = MaterialTranslateCalls ? (int)roundf(float(MaterialCacheHits) / MaterialTranslateCalls * 100) : 0;
	UE_LOGF(LogShaderCompilers, Display, "Material Cache Hits: %d (%d%%)", MaterialCacheHits, HitsPercentage);
}

void FShaderCompilerMaterialCounters::GatherAnalytics(TArray<FAnalyticsEventAttribute>& Attributes)
{
	Attributes.Emplace(TEXT("Material_NumMaterialsCooked"), NumMaterialsCooked);
	Attributes.Emplace(TEXT("Material_MaterialTranslateCalls"), MaterialTranslateCalls);
	Attributes.Emplace(TEXT("Material_MaterialTranslateTimeSec"), MaterialTranslateTotalTimeSec);
	Attributes.Emplace(TEXT("Material_MaterialTranslateTranslationOnlyTimeSec"), MaterialTranslateTranslationOnlyTimeSec);
	Attributes.Emplace(TEXT("Material_MaterialTranslateSerializationOnlyTimeSec"), MaterialTranslateSerializationOnlyTimeSec);
	Attributes.Emplace(TEXT("Material_MaterialCacheHits"), MaterialCacheHits);
}

void FShaderCompilerStats::IncrementMaterialCook()
{
	FScopeLock Lock(&CompileStatsLock);
	MaterialCounters.NumMaterialsCooked++;
}

void FShaderCompilerStats::IncrementMaterialTranslated(double InTotalTime, double InTranslationOnlyTime, double InSerializeTime)
{
	FScopeLock Lock(&CompileStatsLock);
	MaterialCounters.MaterialTranslateCalls++;
	MaterialCounters.MaterialTranslateTotalTimeSec += InTotalTime;
	MaterialCounters.MaterialTranslateTranslationOnlyTimeSec += InTranslationOnlyTime;
	MaterialCounters.MaterialTranslateSerializationOnlyTimeSec += InSerializeTime;
}

void FShaderCompilerStats::IncrementMaterialCacheHit()
{
	FScopeLock Lock(&CompileStatsLock);
	MaterialCounters.MaterialCacheHits++;
}

void FShaderCompilerStats::RegisterCookedShaders(uint32 NumCooked, float CompileTime, EShaderPlatform Platform, const FString MaterialPath, FString PermutationString)
{
	FScopeLock Lock(&CompileStatsLock);
	if(!CompileStats.IsValidIndex(Platform))
	{
		ShaderCompilerStats Stats;
		CompileStats.Insert(Platform, Stats);
	}

	FShaderStats& Stats = CompileStats[Platform].FindOrAdd(MaterialPath);
	Stats.CompileTime += CompileTime;
	bool bFound = false;
	uint64 PermutationHash = FShaderCompilerSinglePermutationStat::GetPermutationHash(PermutationString);
	for (FShaderCompilerSinglePermutationStat& Stat : Stats.PermutationCompilations)
	{
		if (PermutationHash == Stat.PermutationHash)
		{
			bFound = true;
			if (Stat.Cooked != 0)
			{
				Stat.CookedDouble += NumCooked;
				Stats.CookedDouble += NumCooked;
			}
			else
			{
				Stat.Cooked = NumCooked;
				Stats.Cooked += NumCooked;
			}
		}
	}
	if(!bFound)
	{
		Stats.Cooked += NumCooked;
	}
	if (!bFound)
	{
		Stats.PermutationCompilations.Emplace(PermutationHash, 0, NumCooked);
	}
}

void FShaderCompilerStats::RegisterCompiledShaders(uint32 NumCompiled, EShaderPlatform Platform, const FString MaterialPath, FString PermutationString)
{
	FScopeLock Lock(&CompileStatsLock);
	if (!CompileStats.IsValidIndex(Platform))
	{
		ShaderCompilerStats Stats;
		CompileStats.Insert(Platform, Stats);
	}
	FShaderStats& Stats = CompileStats[Platform].FindOrAdd(MaterialPath);

	bool bFound = false;
	uint64 PermutationHash = FShaderCompilerSinglePermutationStat::GetPermutationHash(PermutationString);
	for (FShaderCompilerSinglePermutationStat& Stat : Stats.PermutationCompilations)
	{
		if (PermutationHash == Stat.PermutationHash)
		{
			bFound = true;
			if (Stat.Compiled != 0)
			{
				Stat.CompiledDouble += NumCompiled;
				Stats.CompiledDouble += NumCompiled;
			}
			else
			{
				Stat.Compiled = NumCompiled;
				Stats.Compiled += NumCompiled;
			}
		}
	}
	if(!bFound)
	{
		Stats.Compiled += NumCompiled;
	}


	if (!bFound)
	{
		Stats.PermutationCompilations.Emplace(PermutationHash, NumCompiled, 0);
	}
}

void FShaderCompilerStats::AddDDCMiss(uint32 NumMisses)
{
	Counters.ShaderMapDDCMisses += NumMisses;
}

uint32 FShaderCompilerStats::GetDDCMisses() const
{
	return Counters.ShaderMapDDCMisses;
}

void FShaderCompilerStats::AddDDCHit(uint32 NumHits)
{
	Counters.ShaderMapDDCHits += NumHits;
}

uint32 FShaderCompilerStats::GetDDCHits() const
{
	return Counters.ShaderMapDDCHits;
}

double FShaderCompilerStats::GetTimeShaderCompilationWasActive()
{
	FScopeLock Lock(&CompileStatsLock);
	double Sum = 0;
	for (int32 Idx = 0; Idx < JobLifeTimeIntervals.Num(); ++Idx)
	{
		const TInterval<double>& Existing = JobLifeTimeIntervals[Idx];
		Sum += Existing.Size();
	}
	return Sum;
}


