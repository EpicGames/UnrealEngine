// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	ShaderCompilerThreadRunnable.cpp:
	Implements FShaderCompileThreadRunnableBase and FShaderCompileThreadRunnable.
=============================================================================*/

#include "ShaderCompilerPrivate.h"

#include "Async/ParallelFor.h"
#include "HAL/PlatformFile.h"
#include "HAL/PlatformFileManager.h"
#include "Logging/StructuredLog.h"
#include "Misc/ScopeTryLock.h"
#include "ProfilingDebugging/CookStats.h"
#include "ProfilingDebugging/DiagnosticTable.h"
#include "ProfilingDebugging/LoadTimeTracker.h"
#include "ProfilingDebugging/ScopedTimers.h"
#include "ProfilingDebugging/StallDetector.h"
#include "ShaderCompilerMemoryLimit.h"
#include "ShaderCompileWorkerUtil.h"
#include "AssetCompilingManager.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/QueuedThreadPoolWrapper.h"


float GShaderCompilerTooLongIOThresholdSeconds = 0.3f;
static FAutoConsoleVariableRef CVarShaderCompilerTooLongIOThresholdSeconds(
	TEXT("r.ShaderCompiler.TooLongIOThresholdSeconds"),
	GShaderCompilerTooLongIOThresholdSeconds,
	TEXT("By default, task files for SCW will be read/written sequentially, but if we ever spend more than this time (0.3s by default) doing that, we'll switch to parallel.") \
	TEXT("We don't default to parallel writes as it increases the CPU overhead from the shader compiler."),
	ECVF_Default);

int32 GShaderCompilerCurrentMemoryLimitInMiB = -1;
int32 GShaderCompilerConfiguredMemoryLimitInMiB = 0;
static FAutoConsoleVariableRef CVarShaderCompilerMemoryLimit(
	TEXT("r.ShaderCompiler.MemoryLimit"),
	GShaderCompilerConfiguredMemoryLimitInMiB,
	TEXT("Specifies a memory limit (in MiB) for all ShaderCompileWorker (SCW) processes.") \
	TEXT("If the total memory consumption of all SCW processes exceeds this limit, the editor will start to suspend workers and reschedule compile jobs.") \
	TEXT("By default 0, effectively disabling the limitation. If this is non-zero, it must be greater than or equal to 1024 since shader compilation must be granted at least 1024 MiB of memory in total."),
	ECVF_ReadOnly);

int32 GShaderWorkerStateChangeHeartbeat = 15 * 60;
static FAutoConsoleVariableRef CVarShaderWorkerStateChangeHeartbeat(
	TEXT("r.ShaderCompiler.WorkerHeartbeat"),
	GShaderWorkerStateChangeHeartbeat,
	TEXT("Number of seconds until an unchanged state of compile workers will trigger a heartbeat failure.") \
	TEXT("This will dump the worker states to diagnose hung shader jobs and transfers all jobs of the remote compiler to local workers.") \
	TEXT("This effectively shuts down distributed shader compilation. Default is 15 * 60 seconds."),
	ECVF_Default);

int32 GShaderCompilerAssetMemoryLoanInMiB = -1;

enum class EShaderCompilerWorkerScaleUpApproach : int32
{
	ScaleUpWhenBelowHalfMemoryLimit = 0,
	ScaleUpWhenSpaceForTwoAverageWorkers = 1
};

bool GShaderCompilerWorkerSoftLimit = true;
static FAutoConsoleVariableRef CVarShaderCompilerWorkerSoftLimit(
	TEXT("r.ShaderCompiler.SoftLimit"),
	GShaderCompilerWorkerSoftLimit,
	TEXT("When enabled if not violating the memory, but there isn't sufficient memory for one more worker at the current average memory consumption scale back by one worker.\n") \
	TEXT("This adapts the active worker count to changes in the memory usage across the workers, usually before a violation occurs improving stability.\n"),
	ECVF_Default);

float GShaderCompilerWorkerSoftLimitInWorkers = 0.8f;
static FAutoConsoleVariableRef CVarShaderCompilerWorkerSoftLimitInWorkers(
	TEXT("r.ShaderCompiler.SoftLimitInWorkers"),
	GShaderCompilerWorkerSoftLimitInWorkers,
	TEXT("Multiplier applied to the average memory across shader compiler workers to find the soft limit.\n") \
	TEXT("For example, 1.5 means leave at least enough memory for 1.5 times the memory currently used by an average shader compiler worker\n") \
	TEXT("See r.ShaderCompiler.SoftLimit.\n"),
	ECVF_Default);

float GShaderCompilerMemoryUsedPerWorkerBiasTowardsMax = 0.15f;
static FAutoConsoleVariableRef CVarShaderCompilerWorkerMemoryBiasTowardsMax(
	TEXT("r.ShaderCompiler.MemoryUsedPerWorkerBiasTowardsMax"),
	GShaderCompilerMemoryUsedPerWorkerBiasTowardsMax,
	TEXT("How much to bias the memory estimated required per worker from the current average across the workers towards the current max of any worker.\n") \
	TEXT("Using just the average underestimates the risk of going over the limit when a small number of workers are using a lot of memory.\n")
	TEXT("A value of 0 uses the pure average, while a value of 1 uses the max.\n")
	TEXT("The higher the value, the more conservative scaling back workers is, but the slower overall shader compilation will be.\n"),
	ECVF_Default);

int32 GShaderCompilerWorkerScaleUpApproach = 1;
static FAutoConsoleVariableRef CVarShaderCompilerWorkerScaleUpApproach(
	TEXT("r.ShaderCompiler.WorkerScaleUpApproach"),
	GShaderCompilerWorkerScaleUpApproach,
	TEXT("What approach to use when scaling up or down the active shader compiler workers when running with a memory limit.\n") \
	TEXT("  0: Scale up when memory is below half the limit") \
	TEXT("  1: Scale up when there is enough memory for two more workers, based on the average current worker memory use") \
	TEXT("Only affects behavior when r.ShaderCompiler.MemoryLimit is non-zero"),
	ECVF_Default);

extern bool GShaderCompilerDumpWorkerDiagnostics;

// Configuration to retry shader compile through workers after a worker has been abandoned
static constexpr int32 GSingleThreadedRunsDisabled = -2;
static constexpr int32 GSingleThreadedRunsIncreaseFactor = 8;
static constexpr int32 GSingleThreadedRunsMaxCount = (1 << 24);

static const TCHAR* GWorkerInputFilename = TEXT("WorkerInputOnly.in");
static const TCHAR* GWorkerOutputFilename = TEXT("WorkerOutputOnly.out");

TRACE_DECLARE_MEMORY_COUNTER(AsyncCompilationSCWIndividualMemoryMax, TEXT("AsyncCompilation/SCWIndividualMemory_Max"));
TRACE_DECLARE_MEMORY_COUNTER(AsyncCompilationSCWIndividualMemoryMedian, TEXT("AsyncCompilation/SCWIndividualMemory_Median"));
TRACE_DECLARE_MEMORY_COUNTER(AsyncCompilationSCWMemoryViolationSize, TEXT("AsyncCompilation/SCWMemoryViolationSize"));
TRACE_DECLARE_MEMORY_COUNTER(AsyncCompilationSCWMemory, TEXT("AsyncCompilation/SCWMemory"));
TRACE_DECLARE_MEMORY_COUNTER(AsyncCompilationSCWMemoryLimit, TEXT("AsyncCompilation/SCWMemoryLimit"));
TRACE_DECLARE_MEMORY_COUNTER(AsyncCompilationSCWMemoryLoan, TEXT("AsyncCompilation/SCWMemoryLoan"));

TRACE_DECLARE_INT_COUNTER(AsyncCompilationSCWsActive, TEXT("AsyncCompilation/SCWsActive"));

FShaderCompilerMemoryLimits GetShaderCompilerCurrentMemoryLimits()
{
	FShaderCompilerMemoryLimits Limits = {0};

	Limits.CurrentLimit = GShaderCompilerCurrentMemoryLimitInMiB;
	Limits.BaseLimit = GShaderCompilerConfiguredMemoryLimitInMiB;

	return Limits;
}

void SetShaderCompilerTargetMemoryLimitInMiB(int32 NewLimit)
{
	GShaderCompilerCurrentMemoryLimitInMiB = NewLimit;
}

void SetShaderCompilerAssetMemoryLoanInMiB(int32 NewLoan)
{
	GShaderCompilerAssetMemoryLoanInMiB = NewLoan >= 0 ? NewLoan : 0;
	TRACE_COUNTER_SET(AsyncCompilationSCWMemoryLoan, GShaderCompilerAssetMemoryLoanInMiB);
}

/** Output structure for polling shader process memory violation status. See FShaderCompileThreadRunnable::QueryMemoryLimitViolationStatus(). */
struct FShaderCompilerJobMemoryLimitInfo
{
	/** Size (in bytes) of the shader process memory limitation. */
	int64 MemoryLimit = 0;

	/** Size (in bytes) of the shader process memory usage. When QueryMemoryLimitViolationStatus() returns true, this will be greater than JobMemoryLimit, since the process violated the limitation requirements. */
	int64 MemoryUsed = 0;

	/**
	 * Number of active shader compiler workers.
	 */
	int32 ActiveWorkerCount = 0;
};

/** Information tracked for each shader compile worker process instance. */
struct FShaderCompileWorkerInfo
{
	/** Process handle of the worker app once launched. Invalid handle means no process. */
	FProcHandle WorkerProcess;

	/** Process ID of the worker app once launched. Zero means no process. */
	uint32 WorkerProcessId;

	/** Tracks whether tasks have been issued to the worker. */
	bool bIssuedTasksToWorker;	

	/** Whether the worker has been launched for this set of tasks. */
	bool bLaunchedWorker;

	/** Tracks whether all tasks issued to the worker have been received. */
	bool bComplete;

	/** Whether this worker is available for new jobs. It will be false when shutting down the worker. */
	bool bAvailable;

	/** Time at which the worker started the most recent batch of tasks. */
	double StartTime;

	/** Time at which the worker ended the most recent batch of tasks. */
	double FinishTime;

	/** Jobs that this worker is responsible for compiling. */
	TArray<FShaderCommonCompileJobPtr> QueuedJobs;

	/** Current batch ID to uniquely identify each batch for diagnostics output. */
	uint64 QueuedBatchId;

#if COUNTERSTRACE_ENABLED
	/** Last memory value to avoid duplicate counter logs */
	uint64 StatsLastMemoryUsedValue;

	/** Insights memory counter ID */
	uint16 InsightsCounterId;
#endif

	FShaderCompileWorkerInfo() :
		WorkerProcessId(0),
		bIssuedTasksToWorker(false),
		bLaunchedWorker(false),
		bComplete(false),
		bAvailable(true),
		StartTime(0.0),
		FinishTime(0.0),
		QueuedBatchId(0)
	{
#if COUNTERSTRACE_ENABLED
		// Initialize  memory counter
		static uint64 InsightsUniqueSCWIndex = 0ull;
		StatsLastMemoryUsedValue = MAX_uint64;
		InsightsCounterId = FCountersTrace::OutputInitCounter(
			*FString::Printf(TEXT("AsyncCompilation/SCWIndividualMemory_%llu"), InsightsUniqueSCWIndex++),
			TraceCounterType_Int, TraceCounterDisplayHint_Memory);
#endif
	}

	// warning: not virtual
	~FShaderCompileWorkerInfo()
	{
		TerminateWorkerProcess();
	}

#if COUNTERSTRACE_ENABLED
	void SetMemoryUsedCounter(const uint64 InMemoryUsed)
	{
		if ((!UE_TRACE_CHANNELEXPR_IS_ENABLED(CountersChannel)) || StatsLastMemoryUsedValue == InMemoryUsed)
			return;

		if(InMemoryUsed > 0)
		{
			FCountersTrace::OutputSetValue(InsightsCounterId, static_cast<int64>(InMemoryUsed));
		}
		StatsLastMemoryUsedValue = InMemoryUsed;
	}
#endif

	void TerminateWorkerProcess(bool bAsynchronous = false)
	{
		if (WorkerProcess.IsValid())
		{
			FPlatformProcess::TerminateProc(WorkerProcess, true);
			if (!bAsynchronous)
			{
				while (FPlatformProcess::IsProcRunning(WorkerProcess))
				{
					FPlatformProcess::Sleep(0.01f);
				}
			}
			FPlatformProcess::CloseProc(WorkerProcess);
			WorkerProcess = FProcHandle();
		}
	}

	int32 CloseWorkerProcess()
	{
		int32 ReturnCode = 0;
		if (WorkerProcess.IsValid())
		{
			FPlatformProcess::GetProcReturnCode(WorkerProcess, &ReturnCode);
			FPlatformProcess::CloseProc(WorkerProcess);
			WorkerProcess = FProcHandle();
		}
		return ReturnCode;
	}
};

FShaderCompileThreadRunnableBase::FShaderCompileThreadRunnableBase(FShaderCompilingManager* InManager)
	: WorkerStateHash(0)
	, WorkerStateChangeTimestamp(-1.0)
	, PriorityIndexRange(0, NumShaderCompileJobPriorities - 1)
	, Manager(InManager)
	, Thread(nullptr)
	, bForceFinish(false)
{
	if (GShaderCompilerCurrentMemoryLimitInMiB == -1)
		GShaderCompilerCurrentMemoryLimitInMiB = GShaderCompilerConfiguredMemoryLimitInMiB;
}

bool FShaderCompileThreadRunnableBase::WorkerStateHeartbeat(uint64 InWorkerStateHash)
{
	// Reset worker state hash if it has changed or requested to reset (value of 0)
	const double CurrentTimestamp = FPlatformTime::Seconds();
	if (InWorkerStateHash == 0 || InWorkerStateHash != WorkerStateHash || WorkerStateChangeTimestamp < 0.0)
	{
		WorkerStateHash = InWorkerStateHash;
		WorkerStateChangeTimestamp = CurrentTimestamp;
	}

	// Report warning when hearbeat failed and reset timer
	const double ElapsedTimeSinceWorkerStateChanged = CurrentTimestamp - WorkerStateChangeTimestamp;
	if (ElapsedTimeSinceWorkerStateChanged > (double)GShaderWorkerStateChangeHeartbeat)
	{
		UE_LOGF(LogShaderCompilers, Warning, "No shader compile worker state change in %.2f seconds", ElapsedTimeSinceWorkerStateChanged);
		WorkerStateChangeTimestamp = CurrentTimestamp;
		return false;
	}

	return true;
}

void FShaderCompileThreadRunnableBase::StartThread()
{
	if (Manager->bAllowAsynchronousShaderCompiling && !FPlatformProperties::RequiresCookedData())
	{
		Thread = FRunnableThread::Create(this, GetThreadName(), 0, TPri_Normal, FPlatformAffinity::GetPoolThreadMask());
	}
}

void FShaderCompileThreadRunnableBase::SetPriorityRange(EShaderCompileJobPriority MinPriority, EShaderCompileJobPriority MaxPriority)
{
	FWriteScopeLock LockGuard(PriorityRangeLock);
	PriorityIndexRange.Min = (int32)MinPriority;
	PriorityIndexRange.Max = (int32)MaxPriority;
	check(PriorityIndexRange.Max >= PriorityIndexRange.Min);
}

TInterval<int32> FShaderCompileThreadRunnableBase::GetPriorityRange() const
{
	FReadScopeLock LockGuard(PriorityRangeLock);
	return PriorityIndexRange;
}

FShaderCompileThreadRunnable::FShaderCompileThreadRunnable(FShaderCompilingManager* InManager)
	: FShaderCompileThreadRunnableBase(InManager)
{
	for (uint32 WorkerIndex = 0; WorkerIndex < Manager->NumShaderCompilingThreads; WorkerIndex++)
	{
		WorkerInfos.Add(MakeUnique<FShaderCompileWorkerInfo>());
	}

	bDynamicSCWBudgetEnabled = GConfig->GetBoolOrDefault(TEXT("AssetCompilation"), TEXT("DynamicSCWBudget"), false, GEngineIni);
	DynamicSCWBudgetReserveInMiB = GConfig->GetIntOrDefault(TEXT("AssetCompilation"), TEXT("DynamicSCWBudgetReserveInMiB"), 512, GEngineIni);
	DynamicSCWBudgetGranularityInMiB = FMath::Max(GConfig->GetIntOrDefault(TEXT("AssetCompilation"), TEXT("DynamicSCWBudgetGranularityInMiB"), 256, GEngineIni), 1);	//	Enforce at least 1 to prevent a divide by zero
}

FShaderCompileThreadRunnable::~FShaderCompileThreadRunnable()
{
	FScopeLock WorkerScopeLock(&WorkerInfosLock);
	WorkerInfos.Empty();
}

void FShaderCompileThreadRunnable::OnMachineResourcesChanged()
{
	bool bWaitForWorkersToShutdown = false;
	{
		FScopeLock WorkerScopeLock(&WorkerInfosLock);
		// Set all bAvailable flags back to true
		for (TUniquePtr< FShaderCompileWorkerInfo>& WorkerInfo : WorkerInfos)
		{
			WorkerInfo->bAvailable = true;
		}

		if (Manager->NumShaderCompilingThreads >= static_cast<uint32>(WorkerInfos.Num()))
		{
			while (static_cast<uint32>(WorkerInfos.Num()) < Manager->NumShaderCompilingThreads)
			{
				WorkerInfos.Add(MakeUnique<FShaderCompileWorkerInfo>());
			}
		}
		else
		{
			for (int32 Index = 0; Index < WorkerInfos.Num(); ++Index)
			{
				FShaderCompileWorkerInfo& WorkerInfo = *WorkerInfos[Index];
				bool bReadyForShutdown = WorkerInfo.QueuedJobs.Num() == 0;
				if (bReadyForShutdown)
				{
					WorkerInfos.RemoveAtSwap(Index--);
					if (WorkerInfos.Num() == Manager->NumShaderCompilingThreads)
					{
						break;
					}
				}
			}
			bWaitForWorkersToShutdown = Manager->NumShaderCompilingThreads < static_cast<uint32>(WorkerInfos.Num());
			for (int32 Index = WorkerInfos.Num() - 1;
				static_cast<uint32>(Index) >= Manager->NumShaderCompilingThreads; --Index)
			{
				WorkerInfos[Index]->bAvailable = false;
			}
		}
	}
	const double StartTime = FPlatformTime::Seconds();
	constexpr float MaxDurationToWait = 60.f;
	const double MaxTimeToWait = StartTime + MaxDurationToWait;
	while (bWaitForWorkersToShutdown)
	{
		FPlatformProcess::Sleep(0.01f);
		const double CurrentTime = FPlatformTime::Seconds();
		if (CurrentTime > MaxTimeToWait)
		{
			UE_LOGF(LogShaderCompilers, Warning, "OnMachineResourcesChanged timedout waiting %.0f seconds for WorkerInfos to complete. Workers will remain allocated.",
				(float)(CurrentTime - StartTime));
			break;
		}

		FScopeLock WorkerScopeLock(&WorkerInfosLock);
		for (int32 Index = WorkerInfos.Num() - 1;
			static_cast<uint32>(Index) >= Manager->NumShaderCompilingThreads; --Index)
		{
			FShaderCompileWorkerInfo& WorkerInfo = *WorkerInfos[Index];
			check(!WorkerInfos[Index]->bAvailable); // It should still be set to false from when we changed it above
			bool bReadyForShutdown = WorkerInfo.QueuedJobs.Num() == 0;
			if (bReadyForShutdown)
			{
				WorkerInfos.RemoveAtSwap(Index);
			}
		}
		bWaitForWorkersToShutdown = Manager->NumShaderCompilingThreads < static_cast<uint32>(WorkerInfos.Num());
	}
}

void FShaderCompileThreadRunnable::ForEachPendingJob(const FShaderCompileJobCallback& PendingJobCallback) const
{
	FScopeLock WorkerScopeLock(&WorkerInfosLock);
	for (const TUniquePtr<FShaderCompileWorkerInfo>& WorkerInfo : WorkerInfos)
	{
		for (const FShaderCommonCompileJobPtr& Job : WorkerInfo->QueuedJobs)
		{
			if (!PendingJobCallback(Job.GetReference()))
			{
				return;
			}
		}
	}
}

/** Entry point for the shader compiling thread. */
uint32 FShaderCompileThreadRunnableBase::Run()
{
	Manager->OnShaderCompilingThreadStarted(this);
	{
		LLM_SCOPE_BYTAG(ShaderCompiler);
		check(Manager->bAllowAsynchronousShaderCompiling);
		while (!bForceFinish)
		{
			CompilingLoop();
		}
		UE_LOGF(LogShaderCompilers, Display, "Exiting %ls", GetThreadName());
	}
	Manager->OnShaderCompilingThreadStopped(this);

	return 0;
}

static std::atomic_uint64_t GLocalShaderCompileBatchCounter = 0;

int32 FShaderCompileThreadRunnable::PullTasksFromQueue()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FShaderCompileThreadRunnable::PullTasksFromQueue);

	auto SignalWorkerTasksToBeSubmitted = [this](FShaderCompileWorkerInfo& WorkerInfo, int32 WorkerIndex)
		{
			// Update the worker state as having new tasks that need to be issued					
			// don't reset worker app ID, because the shadercompileworkers don't shutdown immediately after finishing a single job queue.
			WorkerInfo.bIssuedTasksToWorker = false;
			WorkerInfo.bLaunchedWorker = false;
			WorkerInfo.StartTime = FPlatformTime::Seconds();
			WorkerInfo.QueuedBatchId = ++GLocalShaderCompileBatchCounter;

			if (WorkerInfo.FinishTime > 0.0)
			{
				const double WorkerIdleTime = WorkerInfo.StartTime - WorkerInfo.FinishTime;
				GShaderCompilerStats->RegisterLocalWorkerIdleTime(WorkerIdleTime);
				if (Manager->bLogJobCompletionTimes)
				{
					UE_LOGF(LogShaderCompilers, Display, "  Worker (%d/%d) started working after being idle for %fs", WorkerIndex + 1, WorkerInfos.Num(), WorkerIdleTime);
				}
			}
		};

	// Check if memory limitations has been violated and suspend workers as needed
	if (GShaderCompilerCurrentMemoryLimitInMiB > 0)
	{
		CheckMemoryLimitViolation();
	}

	FScopeLock WorkerScopeLock(&WorkerInfosLock); // Must be entered before CompileQueueSection

	int32 NumActiveThreads = 0;
	int32 NumJobsStarted[NumShaderCompileJobPriorities] = { 0 };
	{
		// Enter the critical section so we can access the input and output queues
		FScopeLock Lock(&Manager->CompileQueueSection);

		const int32 NumWorkersToFeed = Manager->bCompilingDuringGame ? Manager->NumShaderCompilingThreadsDuringGame : GetNumberOfAvailableWorkersUnsafe();
		checkf(NumWorkersToFeed > 0, TEXT("Trying to pull tasks in local shader compiling thread, but no workers are available"));

		// Pull tasks from backlogged queue first
		if (!BackloggedJobs.IsEmpty())
		{
			// Try to distribute the work evenly between the workers
			const int32 PriorityIndex = static_cast<int32>(EShaderCompileJobPriority::Normal);
			const int32 NumJobsPerWorker = FMath::DivideAndRoundUp(BackloggedJobs.Num(), NumWorkersToFeed);

			int32 NumWorkersToPickupBacklog = 0;
			int32 NumPickedupBackloggedJobs = 0;

			for (int32 WorkerIndex = 0; WorkerIndex < WorkerInfos.Num(); WorkerIndex++)
			{
				FShaderCompileWorkerInfo& CurrentWorkerInfo = *WorkerInfos[WorkerIndex];

				// If this worker doesn't have any queued jobs, look for more in the input queue
				if (CurrentWorkerInfo.QueuedJobs.Num() == 0 && CurrentWorkerInfo.bAvailable)
				{
					check(!CurrentWorkerInfo.bComplete);

					if (BackloggedJobs.Num() > 0)
					{
						const int32 MaxNumJobs = FMath::Min3(NumJobsPerWorker, BackloggedJobs.Num(), Manager->MaxShaderJobBatchSize);

						// Dequeue backlogged jobs and send them to worker
						CurrentWorkerInfo.QueuedJobs.Reserve(CurrentWorkerInfo.QueuedJobs.Num() + MaxNumJobs);
						for (int32 JobIndex = 0; JobIndex < MaxNumJobs; ++JobIndex)
						{
							CurrentWorkerInfo.QueuedJobs.Add(BackloggedJobs.Pop());
						}
						NumJobsStarted[PriorityIndex] += MaxNumJobs;

						NumPickedupBackloggedJobs += MaxNumJobs;
						NumWorkersToPickupBacklog += 1;

						SignalWorkerTasksToBeSubmitted(CurrentWorkerInfo, WorkerIndex);
					}
				}
			}

			if (NumPickedupBackloggedJobs > 0)
			{
				UE_LOGF(
					LogShaderCompilers, Verbose, "Picked up %d backlogged compile %ls and distributed them over %d %ls",
					NumPickedupBackloggedJobs,
					NumPickedupBackloggedJobs == 1 ? TEXT("job") : TEXT("jobs"),
					NumWorkersToPickupBacklog,
					NumWorkersToPickupBacklog == 1 ? TEXT("worker") : TEXT("workers")
				);
			}
		}

		// Pull tasks from compiling manager queue
		const TInterval<int32> PriorityRange = GetPriorityRange();
		for (int32 PriorityIndex = PriorityRange.Max; PriorityIndex >= PriorityRange.Min; --PriorityIndex)
		{
			const EShaderCompileJobPriority JobPriority = (EShaderCompileJobPriority)PriorityIndex;
			int32 NumPendingJobs = Manager->AllJobs.GetNumPendingJobs(JobPriority);

			// Try to distribute the work evenly between the workers
			const int32 NumJobsPerWorker = FMath::DivideAndRoundUp(NumPendingJobs, NumWorkersToFeed);
			if (NumJobsPerWorker == 0)
			{
				continue;
			}

			for (int32 WorkerIndex = 0; WorkerIndex < WorkerInfos.Num(); WorkerIndex++)
			{
				FShaderCompileWorkerInfo& CurrentWorkerInfo = *WorkerInfos[WorkerIndex];

				// If this worker doesn't have any queued jobs, look for more in the input queue
				if (CurrentWorkerInfo.QueuedJobs.Num() == 0 && CurrentWorkerInfo.bAvailable)
				{
					check(!CurrentWorkerInfo.bComplete);

					NumPendingJobs = Manager->AllJobs.GetNumPendingJobs(JobPriority);
					if (NumPendingJobs > 0)
					{
						UE_LOGF(LogShaderCompilers, Verbose, "Worker (%d/%d): shaders left to compile %i", WorkerIndex + 1, WorkerInfos.Num(), NumPendingJobs);

						int32 MaxNumJobs = 1;
						// high priority jobs go in 1 per "batch", unless the engine is still starting up
						if (PriorityIndex < (int32)EShaderCompileJobPriority::High || Manager->IgnoreAllThrottling())
						{
							MaxNumJobs = FMath::Min3(NumJobsPerWorker, NumPendingJobs, Manager->MaxShaderJobBatchSize);
						}

						NumJobsStarted[PriorityIndex] += Manager->AllJobs.GetPendingJobs(EShaderCompilerWorkerType::LocalThread, JobPriority, 1, MaxNumJobs, CurrentWorkerInfo.QueuedJobs);

						SignalWorkerTasksToBeSubmitted(CurrentWorkerInfo, WorkerIndex);
					}
				}
			}
		}
	}

	for (int32 WorkerIndex = 0; WorkerIndex < WorkerInfos.Num(); WorkerIndex++)
	{
		if (WorkerInfos[WorkerIndex]->QueuedJobs.Num() > 0)
		{
			NumActiveThreads++;
		}
	}

	for (int32 PriorityIndex = 0; PriorityIndex < NumShaderCompileJobPriorities; ++PriorityIndex)
	{
		if (NumJobsStarted[PriorityIndex] > 0)
		{
			UE_LOGF(LogShaderCompilers, Verbose, "Started %d 'Local' shader compile jobs with '%ls' priority",
				NumJobsStarted[PriorityIndex],
				ShaderCompileJobPriorityToString((EShaderCompileJobPriority)PriorityIndex));
		}
	}

	return NumActiveThreads;
}

void FShaderCompileThreadRunnable::PushCompletedJobsToManager()
{
	FScopeLock WorkerScopeLock(&WorkerInfosLock); // Must be entered before CompileQueueSection

	for (int32 WorkerIndex = 0; WorkerIndex < WorkerInfos.Num(); WorkerIndex++)
	{
		FShaderCompileWorkerInfo& CurrentWorkerInfo = *WorkerInfos[WorkerIndex];

		// Add completed jobs to the output queue, which is ShaderMapJobs
		if (CurrentWorkerInfo.bComplete)
		{
			// Enter the critical section so we can access the input and output queues
			FScopeLock Lock(&Manager->CompileQueueSection);

			for (int32 JobIndex = 0; JobIndex < CurrentWorkerInfo.QueuedJobs.Num(); JobIndex++)
			{
				auto& Job = CurrentWorkerInfo.QueuedJobs[JobIndex];

				Manager->ProcessFinishedJob(Job.GetReference(), EShaderCompileJobStatus::Completed);
			}

			const double ElapsedTime = FPlatformTime::Seconds() - CurrentWorkerInfo.StartTime;

			Manager->WorkersBusyTime += ElapsedTime;
			COOK_STAT(AtomicDoubleFetchAdd(ShaderCompilerCookStats::AsyncCompileTimeSec, ElapsedTime, std::memory_order_relaxed));


			CurrentWorkerInfo.FinishTime = FPlatformTime::Seconds();
			CurrentWorkerInfo.bComplete = false;
			CurrentWorkerInfo.QueuedJobs.Empty();
		}
	}
}

void FShaderCompileThreadRunnable::WriteNewTasks()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(ShaderCompiler.WriteNewTasks);
	FScopeLock WorkerScopeLock(&WorkerInfosLock);

	// first, a quick check if anything is needed just to avoid hammering the task graph
	bool bHasTasksToWrite = false;
	for (int32 WorkerIndex = 0, NumWorkers = WorkerInfos.Num(); WorkerIndex < NumWorkers; ++WorkerIndex)
	{
		FShaderCompileWorkerInfo& CurrentWorkerInfo = *WorkerInfos[WorkerIndex];
		if (!CurrentWorkerInfo.bIssuedTasksToWorker && CurrentWorkerInfo.QueuedJobs.Num() > 0)
		{
			bHasTasksToWrite = true;
			break;
		}
	}

	if (!bHasTasksToWrite)
	{
		return;
	}


	auto LoopBody = [this](int32 WorkerIndex)
	{
		// The calling thread holds the WorkerInfosLock and will not modify WorkerInfos, 
		// so we can access it here without entering the lock
		FShaderCompileWorkerInfo& CurrentWorkerInfo = *WorkerInfos[WorkerIndex];

		// Only write tasks once
		if (!CurrentWorkerInfo.bIssuedTasksToWorker && CurrentWorkerInfo.QueuedJobs.Num() > 0)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(ShaderCompiler.WriteNewTasksForWorker);
			CurrentWorkerInfo.bIssuedTasksToWorker = true;

			const FString WorkingDirectory = GetWorkingDirectoryForWorker(WorkerIndex);

			// To make sure that the process waiting for input file won't try to read it until it's ready
			// we use a temp file name during writing.
			FString TransferFileName;
			do
			{
				FGuid Guid;
				FPlatformMisc::CreateGuid(Guid);
				TransferFileName = FPaths::Combine(WorkingDirectory, Guid.ToString());
			} while (IFileManager::Get().FileSize(*TransferFileName) != INDEX_NONE);

			// Write out the file that the worker app is waiting for, which has all the information needed to compile the shader.
			// 'Only' indicates that the worker should keep checking for more tasks after this one
			FArchive* TransferFile = nullptr;

			int32 RetryCount = 0;
			// Retry over the next two seconds if we can't write out the input file
			// Anti-virus and indexing applications can interfere and cause this write to fail
			//@todo - switch to shared memory or some other method without these unpredictable hazards
			while (TransferFile == nullptr && RetryCount < 2000)
			{
				if (RetryCount > 0)
				{
					FPlatformProcess::Sleep(0.01f);
				}
				TransferFile = IFileManager::Get().CreateFileWriter(*TransferFileName, FILEWRITE_EvenIfReadOnly);
				RetryCount++;
				if (TransferFile == nullptr)
				{
					UE_LOGF(LogShaderCompilers, Warning, "Could not create the shader compiler transfer file '%ls', retrying...", *TransferFileName);
				}
			}
			if (TransferFile == nullptr)
			{
				UE_LOGF(LogShaderCompilers, Fatal, "Could not create the shader compiler transfer file '%ls'.", *TransferFileName);
			}
			check(TransferFile);

			GShaderCompilerStats->RegisterJobBatch(CurrentWorkerInfo.QueuedJobs.Num(), FShaderCompilerStats::EExecutionType::Local);
			if (!FShaderCompileWorkerUtil::WriteTasks(CurrentWorkerInfo.QueuedJobs, *TransferFile))
			{
				uint64 TotalDiskSpace = 0;
				uint64 FreeDiskSpace = 0;
				FPlatformMisc::GetDiskTotalAndFreeSpace(TransferFileName, TotalDiskSpace, FreeDiskSpace);
				UE_LOGF(LogShaderCompilers, Error, "Could not write the shader compiler transfer filename to '%ls' (Free Disk Space: %llu.", *TransferFileName, FreeDiskSpace);
			}
			delete TransferFile;

#if 0 // debugging code to dump the worker inputs
			static FCriticalSection ArchiveLock;
			{
				FScopeLock Locker(&ArchiveLock);
				static int ArchivedTransferFileNum = 0;
				FString JobCacheDir = "JobCache";
				FString ArchiveDir = FPaths::ProjectSavedDir() / TEXT("ArchivedWorkerInputs") / JobCacheDir;
				FString ArchiveName = FString::Printf(TEXT("Input-%d"), ArchivedTransferFileNum++);
				FString ArchivePath = ArchiveDir / ArchiveName;
				if (!IFileManager::Get().Copy(*ArchivePath, *TransferFileName))
				{
					UE_LOGF(LogInit, Error, "Could not copy file %ls to %ls", *TransferFileName, *ArchivePath);
					ensure(false);
				}
			}
#endif

			// Change the transfer file name to proper one
			FString ProperTransferFileName = FPaths::Combine(WorkingDirectory, GWorkerInputFilename);
			if (!IFileManager::Get().Move(*ProperTransferFileName, *TransferFileName))
			{
				uint64 TotalDiskSpace = 0;
				uint64 FreeDiskSpace = 0;
				FPlatformMisc::GetDiskTotalAndFreeSpace(TransferFileName, TotalDiskSpace, FreeDiskSpace);
				UE_LOGF(LogShaderCompilers, Error, "Could not rename the shader compiler transfer filename to '%ls' from '%ls' (Free Disk Space: %llu).", *ProperTransferFileName, *TransferFileName, FreeDiskSpace);
			}
		}
	};

	if (bParallelizeIO)
	{
		ParallelFor( TEXT("ShaderCompiler.WriteNewTasks.PF"), WorkerInfos.Num(),1, LoopBody, EParallelForFlags::Unbalanced);
	}
	else
	{
		const double StartIOWork = FPlatformTime::Seconds();
		for (int32 WorkerIndex = 0; WorkerIndex < WorkerInfos.Num(); WorkerIndex++)
		{
			LoopBody(WorkerIndex);
		}

		const double IODuration = FPlatformTime::Seconds() - StartIOWork;
		if (IODuration > GShaderCompilerTooLongIOThresholdSeconds)
		{
			UE_LOGF(LogShaderCompilers, Display, "FShaderCompileThreadRunnable::WriteNewTasks() took too long (%.3f seconds, threshold is %.3f s), will parallelize next time.", IODuration, GShaderCompilerTooLongIOThresholdSeconds);
			bParallelizeIO = true;
		}
	}
}

bool FShaderCompileThreadRunnable::LaunchWorkersIfNeeded()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FShaderCompileThreadRunnable::LaunchWorkersIfNeeded);

	const double CurrentTime = FPlatformTime::Seconds();
	// Limit how often we check for workers running since IsApplicationRunning eats up some CPU time on Windows
	const bool bCheckForWorkerRunning = (CurrentTime - LastCheckForWorkersTime > .1f);
	bool bAbandonWorkers = false;
	uint32_t NumberLaunched = 0;

	if (bCheckForWorkerRunning)
	{
		LastCheckForWorkersTime = CurrentTime;
	}

	FScopeLock WorkerScopeLock(&WorkerInfosLock);
	for (int32 WorkerIndex = 0; WorkerIndex < WorkerInfos.Num(); WorkerIndex++)
	{
		FShaderCompileWorkerInfo& CurrentWorkerInfo = *WorkerInfos[WorkerIndex];
		if (CurrentWorkerInfo.QueuedJobs.Num() == 0)
		{
			// Skip if nothing to do
			// Also, use the opportunity to free OS resources by cleaning up handles of no more running processes
			if (CurrentWorkerInfo.WorkerProcess.IsValid() && !FShaderCompilingManager::IsShaderCompilerWorkerRunning(CurrentWorkerInfo.WorkerProcess))
			{
				CurrentWorkerInfo.CloseWorkerProcess();
			}
			continue;
		}

		if (!CurrentWorkerInfo.WorkerProcess.IsValid() || (bCheckForWorkerRunning && !FShaderCompilingManager::IsShaderCompilerWorkerRunning(CurrentWorkerInfo.WorkerProcess)))
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(FShaderCompileThreadRunnable::LaunchingWorkers);

			// @TODO: dubious design - worker should not be launched unless we know there's more work to do.
			bool bLaunchAgain = true;

			// Detect when the worker has exited due to fatal error
			// bLaunchedWorker check here is necessary to distinguish between 'process isn't running because it crashed' and 'process isn't running because it exited cleanly and the outputfile was already consumed'
			if (CurrentWorkerInfo.WorkerProcess.IsValid())
			{
				// shader compiler exited one way or another, so clear out the stale PID.
				const int32 ReturnCode = CurrentWorkerInfo.CloseWorkerProcess();

				if (CurrentWorkerInfo.bLaunchedWorker)
				{
					const FString OutputFileNameAndPath = FPaths::Combine(GetWorkingDirectoryForWorker(WorkerIndex), GWorkerOutputFilename);

					if (FPlatformFileManager::Get().GetPlatformFile().FileExists(*OutputFileNameAndPath))
					{
						// If the worker is no longer running but it successfully wrote out the output, no need to assert
						bLaunchAgain = false;
					}
					else
					{
						UE_LOGF(LogShaderCompilers, Error, "ShaderCompileWorker terminated unexpectedly, return code %d! Falling back to directly compiling which will be very slow.  Thread %u.", ReturnCode, WorkerIndex);
						FShaderCompileWorkerUtil::LogQueuedCompileJobs(CurrentWorkerInfo.QueuedJobs, -1);

						bAbandonWorkers = true;
						break;
					}
				}
			}

			if (bLaunchAgain)
			{
				constexpr bool bRelativePath = true;
				const FString WorkingDirectory = GetWorkingDirectoryForWorker(WorkerIndex, bRelativePath);

				// Store the handle with this thread so that we will know not to launch it again
				CurrentWorkerInfo.WorkerProcess = Manager->LaunchWorker(WorkingDirectory, Manager->ProcessId, WorkerIndex, GWorkerInputFilename, GWorkerOutputFilename, &CurrentWorkerInfo.WorkerProcessId);
				CurrentWorkerInfo.bLaunchedWorker = true;

				NumberLaunched++;
			}
		}
	}

	const double FinishTime = FPlatformTime::Seconds();
	if (NumberLaunched > 0 && (FinishTime - CurrentTime) >= 10.0)
	{
		UE_LOGF(LogShaderCompilers, Warning, "Performance Warning: It took %f seconds to launch %d ShaderCompileWorkers", FinishTime - CurrentTime, NumberLaunched);
	}

	return bAbandonWorkers;
}

int32 FShaderCompileThreadRunnable::ReadAvailableResults()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(ShaderCompiler.ReadAvailableResults);
	int32 NumProcessed = 0;
	FScopeLock WorkerScopeLock(&WorkerInfosLock);

	// first, a quick check if anything is needed just to avoid hammering the task graph
	bool bHasQueuedJobs = false;
	for (int32 WorkerIndex = 0, NumWorkers = WorkerInfos.Num(); WorkerIndex < NumWorkers; ++WorkerIndex)
	{
		if (WorkerInfos[WorkerIndex]->QueuedJobs.Num() > 0)
		{
			bHasQueuedJobs = true;
			break;
		}
	}

	if (!bHasQueuedJobs)
	{
		return NumProcessed;
	}

	auto LoopBody = [this, &NumProcessed](int32 WorkerIndex)
	{
		// The calling thread holds the WorkerInfosLock and will not modify WorkerInfos, 
		// so we can access it here without entering the lock
		FShaderCompileWorkerInfo& CurrentWorkerInfo = *WorkerInfos[WorkerIndex];

		// Check for available result files
		if (CurrentWorkerInfo.QueuedJobs.Num() > 0)
		{
			// Distributed compiles always use the same directory
			// 'Only' indicates to the worker that it should log and continue checking for the input file after the first one is processed
			const FString OutputFileNameAndPath = FPaths::Combine(GetWorkingDirectoryForWorker(WorkerIndex), GWorkerOutputFilename);

			// In the common case the output file will not exist, so check for existence before opening
			// This is only a win if FileExists is faster than CreateFileReader, which it is on Windows
			if (FPlatformFileManager::Get().GetPlatformFile().FileExists(*OutputFileNameAndPath))
			{
				TRACE_CPUPROFILER_EVENT_SCOPE(FShaderCompileThreadRunnable::ProcessOutputFile);

				if (TUniquePtr<FArchive> OutputFile = TUniquePtr<FArchive>(IFileManager::Get().CreateFileReader(*OutputFileNameAndPath, FILEREAD_Silent)))
				{
					check(!CurrentWorkerInfo.bComplete);
					FShaderCompileWorkerDiagnostics WorkerDiagnostics;

					FShaderCompileWorkerUtil::ReadTasks(CurrentWorkerInfo.QueuedJobs, *OutputFile, GShaderCompilerDumpWorkerDiagnostics ? &WorkerDiagnostics : nullptr);

					if (GShaderCompilerDumpWorkerDiagnostics)
					{
						FString BatchLabel = FString::Printf(TEXT("Local-%" UINT64_FMT), CurrentWorkerInfo.QueuedBatchId);
						GShaderCompilerStats->RegisterWorkerDiagnostics(WorkerDiagnostics, MoveTemp(BatchLabel), CurrentWorkerInfo.QueuedJobs.Num(), CurrentWorkerInfo.WorkerProcessId);
					}

					// Close the output file.
					OutputFile.Reset();

					// Delete the output file now that we have consumed it, to avoid reading stale data on the next compile loop.
					bool bDeletedOutput = IFileManager::Get().Delete(*OutputFileNameAndPath, true, true);
					int32 RetryCount = 0;
					// Retry over the next two seconds if we couldn't delete it
					while (!bDeletedOutput && RetryCount < 200)
					{
						TRACE_CPUPROFILER_EVENT_SCOPE(FShaderCompileThreadRunnable::DeleteOutputFile);

						FPlatformProcess::Sleep(0.01f);
						bDeletedOutput = IFileManager::Get().Delete(*OutputFileNameAndPath, true, true);
						RetryCount++;
					}
					checkf(bDeletedOutput, TEXT("Failed to delete %s!"), *OutputFileNameAndPath);

					CurrentWorkerInfo.bComplete = true;
				}

				FPlatformAtomics::InterlockedIncrement(&NumProcessed);
			}
		}
	};

	if (bParallelizeIO)
	{
		ParallelFor( TEXT("ShaderCompiler.ReadAvailableResults.PF"),WorkerInfos.Num(),1, LoopBody, EParallelForFlags::Unbalanced);
	}
	else 
	{
		const double StartIOWork = FPlatformTime::Seconds();
		for (int32 WorkerIndex = 0; WorkerIndex < WorkerInfos.Num(); WorkerIndex++)
		{
			LoopBody(WorkerIndex);
		}

		const double IODuration = FPlatformTime::Seconds() - StartIOWork;
		if (IODuration > GShaderCompilerTooLongIOThresholdSeconds)
		{
			UE_LOGF(LogShaderCompilers, Display, "FShaderCompileThreadRunnable::ReadAvailableResults() took too long (%.3f seconds, threshold is %.3f s), will parallelize next time.", IODuration, GShaderCompilerTooLongIOThresholdSeconds);
			bParallelizeIO = true;
		}
	}

	return NumProcessed;
}

void FShaderCompileThreadRunnable::CompileDirectlyThroughDll()
{
	// If we aren't compiling through workers, so we can just track the serial time here.
	COOK_STAT(FScopedDurationAtomicTimer CompileTimer(ShaderCompilerCookStats::AsyncCompileTimeSec));

	FScopeLock WorkerScopeLock(&WorkerInfosLock);
	for (int32 WorkerIndex = 0; WorkerIndex < WorkerInfos.Num(); WorkerIndex++)
	{
		FShaderCompileWorkerInfo& CurrentWorkerInfo = *WorkerInfos[WorkerIndex];

		if (CurrentWorkerInfo.QueuedJobs.Num() > 0)
		{
			for (int32 JobIndex = 0; JobIndex < CurrentWorkerInfo.QueuedJobs.Num(); JobIndex++)
			{
				FShaderCommonCompileJob& CurrentJob = *CurrentWorkerInfo.QueuedJobs[JobIndex];
				FShaderCompileUtilities::ExecuteShaderCompileJob(CurrentJob);
			}

			CurrentWorkerInfo.bComplete = true;
		}
	}
}

void FShaderCompileThreadRunnable::PrintWorkerMemoryUsageWithLockTaken()
{
	FPlatformProcessMemoryStats TotalMemoryStats{};
	int32 NumValidWorkers = 0;
	constexpr int64 Gibibyte = 1024 * 1024 * 1024;
	for (int32 Iter = 0, End = WorkerInfos.Num(); Iter < End; Iter++)
	{
		const TUniquePtr<FShaderCompileWorkerInfo>& WorkerInfo = WorkerInfos[Iter];
		FProcHandle ProcHandle = WorkerInfo->WorkerProcess;
		if (!ProcHandle.IsValid())
		{
			continue;
		}
		FPlatformProcessMemoryStats MemoryStats;
		if (FPlatformProcess::TryGetMemoryUsage(ProcHandle, MemoryStats))
		{
			NumValidWorkers++;
			UE_LOGF(LogShaderCompilers, Display,
				"ShaderCompileWorker [%d/%d] MemoryStats:" "\n\t     UsedPhysical %llu (%.2f GiB)" "\n\t PeakUsedPhysical %llu (%.2f GiB)" "\n\t      UsedVirtual %llu (%.2f GiB)" "\n\t  PeakUsedVirtual %llu (%.2f GiB)",
				Iter + 1,
				End,
				MemoryStats.UsedPhysical, double(MemoryStats.UsedPhysical) / Gibibyte,
				MemoryStats.PeakUsedPhysical, double(MemoryStats.PeakUsedPhysical) / Gibibyte,
				MemoryStats.UsedVirtual, double(MemoryStats.UsedVirtual) / Gibibyte,
				MemoryStats.PeakUsedVirtual, double(MemoryStats.PeakUsedVirtual) / Gibibyte
			);
			TotalMemoryStats.UsedPhysical += MemoryStats.UsedPhysical;
			TotalMemoryStats.PeakUsedPhysical += MemoryStats.PeakUsedPhysical;
			TotalMemoryStats.UsedVirtual += MemoryStats.UsedVirtual;
			TotalMemoryStats.PeakUsedVirtual += MemoryStats.PeakUsedVirtual;
		}
		FShaderCompileWorkerUtil::LogQueuedCompileJobs(WorkerInfo->QueuedJobs, -1);
	}

	if (NumValidWorkers > 0)
	{
		UE_LOGF(LogShaderCompilers, Display,
			"Sum of MemoryStats for %d ShaderCompileWorker(s):" "\n\t     UsedPhysical %llu (%.2f GiB)" "\n\t PeakUsedPhysical %llu (%.2f GiB)" "\n\t      UsedVirtual %llu (%.2f GiB)" "\n\t  PeakUsedVirtual %llu (%.2f GiB)",
			NumValidWorkers,
			TotalMemoryStats.UsedPhysical, double(TotalMemoryStats.UsedPhysical) / Gibibyte,
			TotalMemoryStats.PeakUsedPhysical, double(TotalMemoryStats.PeakUsedPhysical) / Gibibyte,
			TotalMemoryStats.UsedVirtual, double(TotalMemoryStats.UsedVirtual) / Gibibyte,
			TotalMemoryStats.PeakUsedVirtual, double(TotalMemoryStats.PeakUsedVirtual) / Gibibyte
		);
	}
}

int32 FShaderCompileThreadRunnable::GetNumberOfWorkers() const
{
	FScopeLock WorkerScopeLock(&WorkerInfosLock);
	return WorkerInfos.Num();
}

int32 FShaderCompileThreadRunnable::GetNumberOfAvailableWorkersUnsafe() const
{
	// Don't lock WorkerScopeLock critical section here, since this function might be called inside an already locked scope, hence the "Unsafe" name
	int32 NumAvailableWorkers = 0;

	for (const TUniquePtr<FShaderCompileWorkerInfo>& WorkerInfo : this->WorkerInfos)
	{
		if (WorkerInfo->bAvailable)
		{
			++NumAvailableWorkers;
		}
	}

	return NumAvailableWorkers;
}

int32 FShaderCompileThreadRunnable::GetNumberOfAvailableWorkers() const
{
	FScopeLock WorkerScopeLock(&WorkerInfosLock);
	return GetNumberOfAvailableWorkersUnsafe();
}

int32 FShaderCompileThreadRunnable::GetNumberOfSuspendedWorkersUnsafe() const
{
	return WorkerInfos.Num() - GetNumberOfAvailableWorkersUnsafe();
}

int32 FShaderCompileThreadRunnable::SuspendWorkersAndBacklogJobs(int32 NumWorkersToSuspend, int32* OutNumBackloggedJobs)
{
	int32 NumSuspendedWorkers = 0;
	int32 NumBackloggedJobs = 0;

	// Before suspending workers, we need to know how many workers are available to ensure there is always at least one worker available
	if (NumWorkersToSuspend > 0)
	{
		FScopeLock WorkerScopeLock(&WorkerInfosLock);
		const int32 NumAvailableWorkers = GetNumberOfAvailableWorkersUnsafe();
		NumWorkersToSuspend = FMath::Min(NumWorkersToSuspend, NumAvailableWorkers - 1);

		if (NumWorkersToSuspend > 0)
		{
			for (int32 WorkerIndex = 0; WorkerIndex < WorkerInfos.Num(); ++WorkerIndex)
			{
				FShaderCompileWorkerInfo& CurrentWorkerInfo = *WorkerInfos[WorkerIndex];
				if (CurrentWorkerInfo.bAvailable)
				{
					// Suspend worker: Terminate its process immediately as we want to free up system resources.
					// Also discard its output file if it has already created one. Otherwise, this file will be linked to the wrong compile jobs.
					CurrentWorkerInfo.bAvailable = false;
					CurrentWorkerInfo.TerminateWorkerProcess();
					DiscardWorkerOutputFile(WorkerIndex);

					// Move its jobs into the backlog queue
					BackloggedJobs.Reserve(BackloggedJobs.Num() + CurrentWorkerInfo.QueuedJobs.Num());
					for (FShaderCommonCompileJobPtr& QueuedJob : CurrentWorkerInfo.QueuedJobs)
					{
						BackloggedJobs.Add(QueuedJob);
					}
					NumBackloggedJobs += CurrentWorkerInfo.QueuedJobs.Num();
					CurrentWorkerInfo.QueuedJobs.Empty();

					// No more workers to suspend? Early exit loop.
					++NumSuspendedWorkers;
					if (NumSuspendedWorkers == NumWorkersToSuspend)
					{
						break;
					}
				}
			}
		}
	}

	if (OutNumBackloggedJobs)
	{
		*OutNumBackloggedJobs = NumBackloggedJobs;
	}

	return NumSuspendedWorkers;
}

int32 FShaderCompileThreadRunnable::ResumeSuspendedWorkers(int32 NumWorkersToResume)
{
	int32 NumResumedWorkers = 0;

	if (NumWorkersToResume > 0)
	{
		FScopeLock WorkerScopeLock(&WorkerInfosLock);
		const int32 NumSuspendedWorkers = GetNumberOfSuspendedWorkersUnsafe();
		NumWorkersToResume = FMath::Min(NumWorkersToResume, NumSuspendedWorkers);

		if (NumWorkersToResume > 0)
		{
			for (int32 WorkerIndex = 0; WorkerIndex < WorkerInfos.Num(); ++WorkerIndex)
			{
				FShaderCompileWorkerInfo& CurrentWorkerInfo = *WorkerInfos[WorkerIndex];
				if (!CurrentWorkerInfo.bAvailable)
				{
					// Resume worker by making it available again. It will pick up jobs next time tasks are pulled from the queue.
					CurrentWorkerInfo.bAvailable = true;

					// No more workers to suspend? Early exit loop.
					++NumResumedWorkers;
					if (NumResumedWorkers == NumWorkersToResume)
					{
						break;
					}
				}
			}
		}
	}

	return NumResumedWorkers;
}

void FShaderCompileThreadRunnable::DiscardWorkerOutputFile(int32 WorkerIndex)
{
	// If the previously suspended worker left a stale output file, delete it now before it gets picked up and is linked to the wrong input jobs
	const FString OutputFileNameAndPath = FPaths::Combine(GetWorkingDirectoryForWorker(WorkerIndex), GWorkerOutputFilename);
	if (IFileManager::Get().FileExists(*OutputFileNameAndPath))
	{
		UE_LOGF(LogShaderCompilers, Verbose, "Discard stale worker output file: %ls", *OutputFileNameAndPath);
		IFileManager::Get().Delete(*OutputFileNameAndPath);
	}
}

FString FShaderCompileThreadRunnable::GetWorkingDirectoryForWorker(int32 WorkerIndex, bool bRelativePath) const
{
	return FPaths::Combine(bRelativePath ? Manager->ShaderBaseWorkingDirectory : Manager->AbsoluteShaderBaseWorkingDirectory, FString::FromInt(WorkerIndex));
}


FShaderCompileMemoryUsage FShaderCompileThreadRunnable::CalcPerWorkerMemoryUsageTotal(const TArray<FWorkerMemoryUsage>& InPerWorkerMemoryUsage) const
{
	FShaderCompileMemoryUsage MemoryUsage{ 0ull, 0ull, 0ll, 0 };


	for (const FWorkerMemoryUsage& WorkerMemUsage : InPerWorkerMemoryUsage)
	{
		MemoryUsage.VirtualMemory += WorkerMemUsage.VirtualMemory;
		MemoryUsage.PhysicalMemory += WorkerMemUsage.PhysicalMemory;

		if (WorkerMemUsage.VirtualMemory > MemoryUsage.MaxWorkerMemory)
		{
			MemoryUsage.MaxWorkerMemory = WorkerMemUsage.VirtualMemory;
		}
	}
	MemoryUsage.ActiveWorkerCount = GetNumberOfAvailableWorkers();

	TRACE_COUNTER_SET(AsyncCompilationSCWMemory, MemoryUsage.VirtualMemory);
	TRACE_COUNTER_SET(AsyncCompilationSCWsActive, MemoryUsage.ActiveWorkerCount);

	return MemoryUsage;
}

void FShaderCompileThreadRunnable::CheckMemoryLimitViolation()
{
	constexpr double kMemoryLimitPollInterval = 0.1; 		// Check every 0.1s, both for suspending and if not suspending, but there are suspended workers resuming.

	const double CurrentTime = FPlatformTime::Seconds();
	const bool bTick = CurrentTime - MemoryMonitoringState.LastTimeOfMemoryLimitPoll > kMemoryLimitPollInterval;
	if (!bTick)
	{
		return;
	}

	{
		FScopeLock PerWorkerMemScopeLock(&PerWorkerMemUsageLock);
		CachePerWorkerMemoryUsage();
	}
	FShaderCompileMemoryUsage TotalSCMemoryUsage = CalcPerWorkerMemoryUsageTotal(PerWorkerMemoryUsage);
	MemoryMonitoringState.LastTimeOfMemoryLimitPoll = CurrentTime;

	//	Query the memory violation information once, whether just checking suspend or resume as well.
	FShaderCompilerJobMemoryLimitInfo LimitInfo;
	bool bHasMemoryViolation = QueryMemoryLimitViolationStatus(TotalSCMemoryUsage, LimitInfo);

	// Check memory limit violations periodically
	{
		FScopeLock PerWorkerMemScopeLock(&PerWorkerMemUsageLock);
		TracePerWorkerSummary(PerWorkerMemoryUsage, bHasMemoryViolation);
	}

	int32 NumWorkersToSuspend = 0;
	const int32 NumberOfAvailableWorkers = GetNumberOfAvailableWorkers();

	if(bHasMemoryViolation)
	{
		// Try to halve the number of workers
		NumWorkersToSuspend = NumberOfAvailableWorkers / 2;
	}
	else
	{
		//	Not violating, check for a soft limit.
		if(GShaderCompilerWorkerSoftLimit)
		{
			if(LimitInfo.ActiveWorkerCount > 1)
			{
				const int64 MemoryUsedPerWorker = GetEstimatedMemoryRequiredPerWorker(TotalSCMemoryUsage);

				//	If there isn't space for one more worker at the current average, suspend one worker.
				//	Resume checks for space for at least two more workers.
				const int64 SoftLimitSize = ((float)MemoryUsedPerWorker * GShaderCompilerWorkerSoftLimitInWorkers);
				const int64 MemoryForAllWorkersPlusSoftLimit = LimitInfo.MemoryUsed + SoftLimitSize;
				if(MemoryForAllWorkersPlusSoftLimit >= LimitInfo.MemoryLimit)
				{
					NumWorkersToSuspend = 1;
					bHasMemoryViolation = true;
				}
			}
		}
	}

	if(NumWorkersToSuspend)
	{
		int32 NumBackloggedJobs = 0;
		const int32 NumSuspendedWorkers = SuspendWorkersAndBacklogJobs(NumWorkersToSuspend, &NumBackloggedJobs);
		if (NumSuspendedWorkers > 0)
		{
			UE_LOGFMT_NSLOC(
				LogShaderCompilers, Display, "ShaderCompilers", "SuspendingWorkers",
				"Shader compiler memory usage of {MemoryUsed} MiB, limit of {MemoryLimit} MiB: " \
				"Backlogged {BackloggedJobs} compile {BackloggedJobsName} from {SuspendedWorkers} suspended {SuspendedWorkersName} ({ActiveWorkerCount}/{TotalWorkerCount} active)",
				("MemoryUsed", static_cast<int32>(LimitInfo.MemoryUsed / 1024 / 1024)),
				("MemoryLimit", static_cast<int32>(LimitInfo.MemoryLimit / 1024 / 1024)),
				("BackloggedJobs", NumBackloggedJobs),
				("BackloggedJobsName", NumBackloggedJobs == 1 ? TEXT("job") : TEXT("jobs")),
				("SuspendedWorkers", NumSuspendedWorkers),
				("SuspendedWorkersName", NumSuspendedWorkers == 1 ? TEXT("worker") : TEXT("workers")),
				("ActiveWorkerCount", GetNumberOfAvailableWorkers()),
				("TotalWorkerCount", GetNumberOfWorkers())
			);
			MemoryMonitoringState.bHasFailedToSuspendWorkers = false;
			LimitInfo.ActiveWorkerCount -= NumSuspendedWorkers;
		}
		else if (!MemoryMonitoringState.bHasFailedToSuspendWorkers)
		{
			UE_LOGFMT_NSLOC(
				LogShaderCompilers, Warning, "ShaderCompilers", "SuspendingWorkersFailed",
				"Shader compiler memory usage {MemoryUsed} MiB, limit of {MemoryLimit} MiB, but cannot suspend any more workers",
				("MemoryUsed", static_cast<int32>(LimitInfo.MemoryUsed / 1024 / 1024)),
				("MemoryLimit", static_cast<int32>(LimitInfo.MemoryLimit / 1024 / 1024))
			);
			MemoryMonitoringState.bHasFailedToSuspendWorkers = true; // Don't show this warning again unless we were able to suspend workers again
		}
		MemoryMonitoringState.LastTimeOfSuspendingOrResumingWorkers = CurrentTime;

		//	Return having suspended, or tried to suspend some workers.
		return;
	}

	const int32 NumberOfWorkers = GetNumberOfWorkers();
	int32 NumAvailableWorkers = GetNumberOfAvailableWorkers();
	if(NumAvailableWorkers == NumberOfWorkers)
	{
		return;
	}

	//	There is at least one suspended worker, check if we can resume any.
	int32 NumWorkersToResume = 0;
	int32 NumResumedWorkers = 0;
	if(GShaderCompilerWorkerScaleUpApproach <= static_cast<int32>(EShaderCompilerWorkerScaleUpApproach::ScaleUpWhenBelowHalfMemoryLimit))
	{

		// If we are below half of our memory limit, resume 50% of available workers.
		// This approach suspends workers from 100% to 50% and then resumes them back up to 75%.
		if (LimitInfo.MemoryUsed < LimitInfo.MemoryLimit / 2)
		{
			//	Number of workers to resume is half the currently inactive workers, with 1 added before the divide to always ensure at least one worker
			//	is attempted to be woken.
			NumWorkersToResume = ((NumberOfWorkers - NumAvailableWorkers)+1) / 2;
			NumResumedWorkers = ResumeSuspendedWorkers(NumWorkersToResume);
		}
	}
	else
	{
		ensure(GShaderCompilerWorkerScaleUpApproach == static_cast<int32>(EShaderCompilerWorkerScaleUpApproach::ScaleUpWhenSpaceForTwoAverageWorkers));

		//	Scale back up if there looks to be enough memory to fit two average size workers based on current worker memory usage.
		NumWorkersToResume = 0;
		const int64 MemoryAvailable = LimitInfo.MemoryLimit;
		if (MemoryAvailable > 0)
		{
			//	Set a memory per worker based on the config, adpating if there are running workers.
			const int32 EffectiveActiveWorkerCount = LimitInfo.ActiveWorkerCount;
			const int64 MemoryUsedPerWorker = GetEstimatedMemoryRequiredPerWorker(TotalSCMemoryUsage);
			if(MemoryAvailable > (MemoryUsedPerWorker*(EffectiveActiveWorkerCount+2)))
			{
				NumWorkersToResume = 1;
			}

			if(NumWorkersToResume)
			{
				NumResumedWorkers = ResumeSuspendedWorkers(NumWorkersToResume);
			}
		}
	}

	if ((NumWorkersToResume > 0) && (NumResumedWorkers > 0))
	{
		MemoryMonitoringState.LastTimeOfSuspendingOrResumingWorkers = CurrentTime;
		UE_LOGFMT_NSLOC(
			LogShaderCompilers, Display, "ShaderCompilers", "ResumingWorkers",
			"Resumed {ResumedWorkers} suspended {ResumedWorkersName} memory usage {MemoryUsed} MiB, limit of {MemoryLimit} MiB ({ActiveWorkerCount}/{TotalWorkerCount} active)",
			("ResumedWorkers", NumResumedWorkers),
			("ResumedWorkersName", NumResumedWorkers == 1 ? TEXT("worker") : TEXT("workers")),
			("MemoryUsed", static_cast<int32>(LimitInfo.MemoryUsed / 1024 / 1024)),
			("MemoryLimit", static_cast<int32>(LimitInfo.MemoryLimit / 1024 / 1024)),
			("ActiveWorkerCount", GetNumberOfAvailableWorkers()),
			("TotalWorkerCount", NumberOfWorkers)
		);
	}
}

int64 FShaderCompileThreadRunnable::GetEstimatedMemoryRequiredPerWorker(const FShaderCompileMemoryUsage& MemoryUsage)
{
	const float BiasTowardsMax = FMath::Clamp(GShaderCompilerMemoryUsedPerWorkerBiasTowardsMax, 0.0f, 1.0f);
	int64 MemoryUsedPerWorker = (MemoryUsage.VirtualMemory / (MemoryUsage.ActiveWorkerCount ? MemoryUsage.ActiveWorkerCount : 1));
	MemoryUsedPerWorker = FMath::Lerp(MemoryUsedPerWorker, MemoryUsage.MaxWorkerMemory, BiasTowardsMax);

	return MemoryUsedPerWorker;
}

bool FShaderCompileThreadRunnable::QueryMemoryLimitViolationStatus(const FShaderCompileMemoryUsage& InSCMemoryUsage, FShaderCompilerJobMemoryLimitInfo& OutInfo)
{
	const int64 MemoryLimitInBytes = static_cast<int64>(GShaderCompilerCurrentMemoryLimitInMiB) * 1024ull * 1024ull;

	OutInfo.MemoryLimit = MemoryLimitInBytes;
	OutInfo.MemoryUsed = InSCMemoryUsage.VirtualMemory;
	OutInfo.ActiveWorkerCount = InSCMemoryUsage.ActiveWorkerCount;

	return InSCMemoryUsage.VirtualMemory >= static_cast<uint64>(MemoryLimitInBytes);
}

bool FShaderCompileThreadRunnable::PrintWorkerMemoryUsage(bool bAllowToWaitForLock)
{
	if (bAllowToWaitForLock)
	{
		FScopeLock WorkerScopeLock(&WorkerInfosLock);
		PrintWorkerMemoryUsageWithLockTaken();
		return true;
	}
	else
	{
		FScopeTryLock WorkerScopeLock(&WorkerInfosLock);
		if (WorkerScopeLock.IsLocked())
		{
			PrintWorkerMemoryUsageWithLockTaken();
			return true;
		}
		return false;
	}
}

void FShaderCompileThreadRunnable::CachePerWorkerMemoryUsage()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FShaderCompileThreadRunnable::CachePerWorkerMemoryUsage);

	const double CurrentTime = FPlatformTime::Seconds();
	const bool bCheckPerworkerMemoryUsage = (CurrentTime - LastCheckForWorkersMemoryUsageTime > .1f);
	if (!bCheckPerworkerMemoryUsage)
	{
		return;
	}
	LastCheckForWorkersMemoryUsageTime = CurrentTime;

	FScopeLock WorkerScopeLock(&WorkerInfosLock);
	if (WorkerInfos.Num() <= 0)
	{
		PerWorkerMemoryUsage.Reset();
		return;
	}

	PerWorkerMemoryUsage.Empty(WorkerInfos.Num());

	FResourceProcessTreeMemory ProcessTreeMemory;
	ProcessTreeMemory.CacheSystemProcessIds();

	int32 WorkerIndex = 0;
	for (const TUniquePtr<FShaderCompileWorkerInfo>& WorkerInfo : WorkerInfos)
	{
		if (WorkerInfo->WorkerProcess.IsValid() && WorkerInfo->WorkerProcessId != 0)
		{
			ProcessTreeMemory.Reset();
			ProcessTreeMemory.AddRootProcessId(WorkerInfo->WorkerProcessId);

			FPlatformProcessMemoryStats MemoryStats;
			if (ProcessTreeMemory.TryGetMemoryUsage(MemoryStats))
			{
				// Virtual memory is committed memory on Windows.
				PerWorkerMemoryUsage.Add(
					{
						.WorkerIndex = WorkerIndex,
						.VirtualMemory = MemoryStats.UsedVirtual,
						.PhysicalMemory = MemoryStats.UsedPhysical
					});

				++WorkerIndex;
				continue;
			}
		}

		// Pack the array with 0 so we match the worker array count
		PerWorkerMemoryUsage.Add(
			{
				.WorkerIndex = WorkerIndex,
				.VirtualMemory = 0,
				.PhysicalMemory = 0
			});

		++WorkerIndex;
	}

	check(PerWorkerMemoryUsage.Num() == WorkerInfos.Num());
}

uint64 FShaderCompileThreadRunnable::GetPerWorkerVirtualMemoryMedian(const TArray<FWorkerMemoryUsage>& InWorkerMemoryUsage) const
{
	TArray<uint64> MemoryValues;
	MemoryValues.Reserve(InWorkerMemoryUsage.Num());

	// zero is valid but ignored (inactive/non-violating worker)
	for (const FWorkerMemoryUsage& MemoryUsage : InWorkerMemoryUsage)
	{
		if (MemoryUsage.VirtualMemory > 0)
			MemoryValues.Add(MemoryUsage.VirtualMemory);
	}

	const int32 Count = MemoryValues.Num();
	if (Count <= 0)
		return 0ull;

	MemoryValues.Sort([](const uint64& InLeft, const uint64& InRight)
		{ return InLeft < InRight; }
	);

	const int32 Index = Count / 2;
	const bool bIsOdd = Count % 2 == 0;
	return bIsOdd ? (MemoryValues[Index - 1] + MemoryValues[Index]) / 2ull : MemoryValues[Index];
}

void FShaderCompileThreadRunnable::TracePerWorkerSummary(const TArray<FWorkerMemoryUsage>& InPerWorkerMemoryUsage, const bool bInHasMemoryViolation) const
{
	if (InPerWorkerMemoryUsage.Num() > 0)
	{
		uint64 WorkersMaxVirtualMemory = 0ull;

#if COUNTERSTRACE_ENABLED
		FScopeLock WorkerScopeLock(&WorkerInfosLock);
#endif

		for (auto WorkerMemIt = InPerWorkerMemoryUsage.CreateConstIterator(); WorkerMemIt; ++WorkerMemIt)
		{
			const FWorkerMemoryUsage& WorkerMemoryUsageRef = *WorkerMemIt;

			if (WorkerMemoryUsageRef.VirtualMemory > WorkersMaxVirtualMemory)
				WorkersMaxVirtualMemory = WorkerMemoryUsageRef.VirtualMemory;

#if COUNTERSTRACE_ENABLED
			check(WorkerMemoryUsageRef.WorkerIndex >= 0 && WorkerMemoryUsageRef.WorkerIndex < WorkerInfos.Num());
			WorkerInfos[WorkerMemoryUsageRef.WorkerIndex]->SetMemoryUsedCounter(WorkerMemoryUsageRef.VirtualMemory);
#endif
		}

#if COUNTERSTRACE_ENABLED
		WorkerScopeLock.Unlock();
#endif

		const uint64 MedianVirtualMemory = WorkersMaxVirtualMemory > 0ull ? GetPerWorkerVirtualMemoryMedian(InPerWorkerMemoryUsage) : 0ull;

		TRACE_COUNTER_SET(AsyncCompilationSCWIndividualMemoryMax, WorkersMaxVirtualMemory);
		TRACE_COUNTER_SET(AsyncCompilationSCWIndividualMemoryMedian, MedianVirtualMemory);

		if (bInHasMemoryViolation)
		{
			UE_LOGFMT_NSLOC(
				LogShaderCompilers, Display, "ShaderCompilers", "PerWorkerMemoryViolation",
				"Shader Compiler Worker memory violation. " \
				"Per-worker Summary: Median {MedianVirtualMemoryBytes} bytes. Max {MaxVirtualMemoryBytes} bytes",
				("MedianVirtualMemoryBytes", static_cast<int32>(MedianVirtualMemory)),
				("MaxVirtualMemoryBytes", static_cast<int32>(WorkersMaxVirtualMemory))
			);
		}
	}
	else if (bInHasMemoryViolation)
	{
		UE_LOGFMT_NSLOC(
			LogShaderCompilers, Display, "ShaderCompilers", "FailedRetrievePerWorkerMemoryViolation",
			"Shader Compiler Worker memory violation: Failed to retrieve per-worker summary breakdown."
		);
	}
}

FShaderCompileMemoryUsage FShaderCompileThreadRunnable::GetExternalWorkerMemoryUsage()
{
	TArray<FWorkerMemoryUsage> RecentWorkerMemoryUsage = GetRecentPerWorkerMemoryUsage();
	FShaderCompileMemoryUsage MemoryUsage = CalcPerWorkerMemoryUsageTotal(RecentWorkerMemoryUsage);

	return MemoryUsage;
}

static void ReportShaderCompileWorkerDiagnostics(const TArray<TUniquePtr<FShaderCompileWorkerInfo>>& InWorkerInfos)
{
	UE_LOGF(LogShaderCompilers, Display, "======= ShaderCompileWorker Diagnostics =======");

	FString JobDiagnostics;

	for (int32 WorkerIndex = 0; WorkerIndex < InWorkerInfos.Num(); ++WorkerIndex)
	{
		const TUniquePtr<FShaderCompileWorkerInfo>& WorkerInfo = InWorkerInfos[WorkerIndex];
		if (!WorkerInfo->QueuedJobs.IsEmpty())
		{
			JobDiagnostics.Empty();

			for (int32 JobIndex = 0; JobIndex < WorkerInfo->QueuedJobs.Num(); ++JobIndex)
			{
				WorkerInfo->QueuedJobs[JobIndex]->AppendDiagnostics(JobDiagnostics, JobIndex, WorkerInfo->QueuedJobs.Num(), TEXT("  "));
			}

			UE_LOGF(
				LogShaderCompilers, Display, "Worker [%d/%d]: bAvailable=%d, bComplete=%d, bIssuedTasksToWorker=%d, bLaunchedWorker=%d\n%ls",
				WorkerIndex + 1, InWorkerInfos.Num(), (int32)WorkerInfo->bAvailable, (int32)WorkerInfo->bComplete, (int32)WorkerInfo->bIssuedTasksToWorker, (int32)WorkerInfo->bLaunchedWorker, *JobDiagnostics
			);
		}
	}
}

int32 FShaderCompileThreadRunnable::CompilingLoop()
{

	// Generate hash over all worker states to detect hung shader compile jobs
	TMemoryHasher<FXxHash64Builder, FXxHash64> WorkerStateHasher;
	bool bHasAnyJobs = false;

	for (const TUniquePtr<FShaderCompileWorkerInfo>& WorkerInfo : WorkerInfos)
	{
		WorkerStateHasher << WorkerInfo->bAvailable << WorkerInfo->bComplete << WorkerInfo->bIssuedTasksToWorker << WorkerInfo->bLaunchedWorker;
		for (const auto& Job : WorkerInfo->QueuedJobs)
		{
			bHasAnyJobs = true;
			WorkerStateHasher << Job->InputHash;
		}
	}

	if (!WorkerStateHeartbeat(bHasAnyJobs ? WorkerStateHasher.Finalize().Hash : 0))
	{
		ReportShaderCompileWorkerDiagnostics(WorkerInfos);
	}

	// push completed jobs to Manager->ShaderMapJobs before asking for new ones, so we can free the workers now and avoid them waiting a cycle
	PushCompletedJobsToManager();

	// Grab more shader compile jobs from the input queue
	const int32 NumActiveThreads = PullTasksFromQueue();

	if (NumActiveThreads == 0 && Manager->bAllowAsynchronousShaderCompiling)
	{
		// Yield while there's nothing to do
		// Note: sleep-looping is bad threading practice, wait on an event instead!
		// The shader worker thread does it because it needs to communicate with other processes through the file system
		FPlatformProcess::Sleep(.010f);
	}

	if (Manager->bAllowCompilingThroughWorkers)
	{
		// Write out the files which are input to the shader compile workers
		WriteNewTasks();

		// Launch shader compile workers if they are not already running
		// Workers can time out when idle so they may need to be relaunched
		bool bAbandonWorkers = LaunchWorkersIfNeeded();

		if (bAbandonWorkers)
		{
			// Immediately terminate all worker processes and delete any output files they may have generated;
			// we will re-run all these jobs locally instead.
			for (int32 WorkerIndex = 0; WorkerIndex < WorkerInfos.Num(); ++WorkerIndex)
			{
				WorkerInfos[WorkerIndex]->TerminateWorkerProcess();
				DiscardWorkerOutputFile(WorkerIndex);
			}

			// Fall back to local compiles if the SCW crashed.
			// This is nasty but needed to work around issues where message passing through files to SCW is unreliable on random PCs
			Manager->bAllowCompilingThroughWorkers = false;

			// Try to recover from abandoned workers after a certain amount of single-threaded compilations
			if (Manager->NumSingleThreadedRunsBeforeRetry == GSingleThreadedRunsIdle)
			{
				// First try to recover, only run single-threaded approach once
				Manager->NumSingleThreadedRunsBeforeRetry = 1;
			}
			else if (Manager->NumSingleThreadedRunsBeforeRetry > GSingleThreadedRunsMaxCount)
			{
				// Stop retry approach after too many retries have failed
				Manager->NumSingleThreadedRunsBeforeRetry = GSingleThreadedRunsDisabled;
			}
			else
			{
				// Next time increase runs by factor X
				Manager->NumSingleThreadedRunsBeforeRetry *= GSingleThreadedRunsIncreaseFactor;
			}
		}
		else
		{
			// Read files which are outputs from the shader compile workers
			int32 NumProcessedResults = ReadAvailableResults();
			if (NumProcessedResults == 0)
			{
				// Reduce filesystem query rate while actively waiting for results.
				FPlatformProcess::Sleep(0.1f);
			}
		}
	}
	else
	{
		// Execute all pending worker tasks single-threaded
		CompileDirectlyThroughDll();

		// If single-threaded mode was enabled by an abandoned worker, try to recover after the given amount of runs
		if (Manager->NumSingleThreadedRunsBeforeRetry > 0)
		{
			Manager->NumSingleThreadedRunsBeforeRetry--;
			if (Manager->NumSingleThreadedRunsBeforeRetry == 0)
			{
				UE_LOGF(LogShaderCompilers, Display, "Retry shader compiling through workers.");
				Manager->bAllowCompilingThroughWorkers = true;
			}
		}
	}

	if(NumActiveThreads)
	{
		{
			FScopeLock PerWorkerMemScopeLock(&PerWorkerMemUsageLock);
			CachePerWorkerMemoryUsage();
		}
		FShaderCompileMemoryUsage TotalSCMemoryUsage = CalcPerWorkerMemoryUsageTotal(PerWorkerMemoryUsage);

		if(bDynamicSCWBudgetEnabled)
		{
			UpdateDynamicSCWMemoryBudget(TotalSCMemoryUsage);
		}
	}

	return NumActiveThreads;
}

void FShaderCompileThreadRunnable::UpdateDynamicSCWMemoryBudget(const FShaderCompileMemoryUsage& SCMemoryUsage)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UpdateDynamicSCWMemoryBudget);

	const FPlatformMemoryStats MemoryStats = FPlatformMemory::GetStats();
	const int64 SystemAvailableMemory = (int64)MemoryStats.AvailableVirtual;

	//	Estimate what memory can be used currently for shader compilation.
	//	System currently free
	//	Shader compiler currently using
	//	Asset compilation... what the current allocation is, not the estimate.
	//	AllocationGranularity is used to reduce the frequency of changes. Small

	int64 Reserve = static_cast<int64>(DynamicSCWBudgetReserveInMiB)*1024ll*1024ll;
	Reserve += static_cast<int64>(GShaderCompilerAssetMemoryLoanInMiB)*1024ll*1024ll;	//	Include any additional requested. (Typically by asset compilation.)
	const int64 AllocationGranularity = static_cast<int64>(DynamicSCWBudgetGranularityInMiB)*1024ll*1024ll;

	FQueuedThreadPoolWrapper* AssetThreadPool = static_cast<FQueuedThreadPoolWrapper*>(FAssetCompilingManager::Get().GetThreadPool());
	if(AssetThreadPool)
	{
		const int64 QueuedAssetMemory = AssetThreadPool->GetEstimatedMemory();
		Reserve -= QueuedAssetMemory;
	}
	int64 MemoryAvailable = SystemAvailableMemory - Reserve;	//	Note, can be negative at this point.

	int64 ShaderCompilationUsage = SCMemoryUsage.VirtualMemory;
	MemoryAvailable += ShaderCompilationUsage;	//	Count the currently in use SCW memory as available for counting how much memory for the SCWs

	//	Log if below 3GiB. This is based on the original budget of 3500. When things are consistently below that, shader compilation speed will be slow.
	constexpr int64 LowSCWMemoryThreshold = (3ll * 1024ll * 1024ll * 1024ll);
	if(MemoryAvailable < LowSCWMemoryThreshold)
	{
		static int64 MinMemoryAvailable = LowSCWMemoryThreshold;

		if(MemoryAvailable < MinMemoryAvailable)
		{
			UE_LOGF(LogShaderCompilers, Display, "Memory available for shader compilation is low, %lldMiB. Shader compilation speed will be slow.", MemoryAvailable / (1024ull * 1024ull));
			MinMemoryAvailable = MemoryAvailable;
		}
	}

	static int64 CurrentSCWBudget = 0;
	int64 DesiredSWCBudget = MemoryAvailable;
	//	Quantize the desired amount to reduce changes to be significant.
	DesiredSWCBudget = (DesiredSWCBudget / AllocationGranularity) * AllocationGranularity;
	//	Limit to twice the configured max.
	const FShaderCompilerMemoryLimits SCWLimits = GetShaderCompilerCurrentMemoryLimits();
	const int64 MaxSCWBudget = ((int64)SCWLimits.BaseLimit) * 1024ll * 1024ll * 2ll;
	DesiredSWCBudget = FMath::Clamp<int64>(DesiredSWCBudget, Reserve, MaxSCWBudget);

	if(DesiredSWCBudget != CurrentSCWBudget)
	{
		CurrentSCWBudget = DesiredSWCBudget;
		int32 CurrentSCWBudgetInMiB = FMath::Min((CurrentSCWBudget / (1024ll*1024ll)), INT32_MAX);
		SetShaderCompilerTargetMemoryLimitInMiB(CurrentSCWBudgetInMiB);
		TRACE_COUNTER_SET(AsyncCompilationSCWMemoryLimit, CurrentSCWBudget);
	}
}
