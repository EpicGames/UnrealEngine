// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	ShaderCompilerUtil.cpp: Implements FShaderCompileUtilities.
=============================================================================*/

#include "ShaderCompilerPrivate.h"

#include "HAL/PlatformFile.h"
#include "HAL/PlatformFileManager.h"
#include "Interfaces/IShaderFormat.h"
#include "Interfaces/ITargetPlatform.h"
#include "Interfaces/ITargetPlatformManagerModule.h"
#include "Misc/Compression.h"
#include "Misc/FileHelper.h"
#include "Misc/MessageDialog.h"
#include "Serialization/MemoryWriter.h"
#include "ShaderDiagnostics.h"

void FShaderCompileUtilities::ExecuteShaderCompileJob(FShaderCommonCompileJob& Job)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FShaderCompileUtilities::ExecuteShaderCompileJob);

	check(Job.GetStatus() != EShaderCompileJobStatus::Completed);

	static ITargetPlatformManagerModule& TPM = GetTargetPlatformManagerRef();
	auto* SingleJob = Job.GetSingleShaderJob();
	const TArray<const IShaderFormat*> ShaderFormats = TPM.GetShaderFormats();
	if (SingleJob)
	{
		const FName Format = (SingleJob->Input.ShaderFormat != NAME_None) ? SingleJob->Input.ShaderFormat : LegacyShaderPlatformToShaderFormat(EShaderPlatform(SingleJob->Input.Target.Platform));
		CompileShader(ShaderFormats, *SingleJob);
	}
	else
	{
		FShaderPipelineCompileJob* PipelineJob = Job.GetShaderPipelineJob();
		check(PipelineJob);

		EShaderPlatform Platform = (EShaderPlatform)PipelineJob->StageJobs[0]->Input.Target.Platform;
		const FName Format = LegacyShaderPlatformToShaderFormat(Platform);

		// Verify same platform on all stages
		for (int32 Index = 1; Index < PipelineJob->StageJobs.Num(); ++Index)
		{
			auto SingleStage = PipelineJob->StageJobs[Index];
			if (!SingleStage)
			{
				UE_LOGF(LogShaderCompilers, Fatal, "Can't nest Shader Pipelines inside Shader Pipeline '%ls'!", PipelineJob->Key.ShaderPipeline->GetName());
			}
			else if (Platform != SingleStage->Input.Target.Platform)
			{
				UE_LOGF(LogShaderCompilers, Fatal, "Mismatched Target Platform %ls while compiling Shader Pipeline '%ls'.", *Format.GetPlainNameString(), PipelineJob->Key.ShaderPipeline->GetName());
			}
		}

		CompileShaderPipeline(ShaderFormats, PipelineJob);
	}

	Job.UpdateStatus(EShaderCompileJobStatus::Completed, EShaderCompileJobCompletionMethod::LocalExecution);
}

FArchive* FShaderCompileUtilities::CreateFileHelper(const FString& Filename)
{
	// TODO: This logic came from FShaderCompileThreadRunnable::WriteNewTasks().
	// We can't avoid code duplication unless we refactored the local worker too.

	FArchive* File = nullptr;
	int32 RetryCount = 0;
	// Retry over the next two seconds if we can't write out the file.
	// Anti-virus and indexing applications can interfere and cause this to fail.
	while (File == nullptr && RetryCount < 200)
	{
		if (RetryCount > 0)
		{
			FPlatformProcess::Sleep(0.01f);
		}
		File = IFileManager::Get().CreateFileWriter(*Filename, FILEWRITE_EvenIfReadOnly);
		RetryCount++;
	}
	if (File == nullptr)
	{
		File = IFileManager::Get().CreateFileWriter(*Filename, FILEWRITE_EvenIfReadOnly | FILEWRITE_NoFail);
	}
	checkf(File, TEXT("Failed to create file %s!"), *Filename);
	return File;
}

void FShaderCompileUtilities::MoveFileHelper(const FString& To, const FString& From)
{
	IPlatformFile& PlatformFile = FPlatformFileManager::Get().GetPlatformFile();

	if (PlatformFile.FileExists(*From))
	{
		FString DirectoryName;
		int32 LastSlashIndex;
		if (To.FindLastChar('/', LastSlashIndex))
		{
			DirectoryName = To.Left(LastSlashIndex);
		} else
		{
			DirectoryName = To;
		}

		// TODO: This logic came from FShaderCompileThreadRunnable::WriteNewTasks().
		// We can't avoid code duplication unless we refactored the local worker too.

		bool Success = false;
		int32 RetryCount = 0;
		// Retry over the next two seconds if we can't move the file.
		// Anti-virus and indexing applications can interfere and cause this to fail.
		while (!Success && RetryCount < 200)
		{
			if (RetryCount > 0)
			{
				FPlatformProcess::Sleep(0.01f);
			}

			// MoveFile does not create the directory tree, so try to do that now...
			Success = PlatformFile.CreateDirectoryTree(*DirectoryName);
			if (Success)
			{
				Success = PlatformFile.MoveFile(*To, *From);
			}
			RetryCount++;
		}
		checkf(Success, TEXT("Failed to move file %s to %s!"), *From, *To);
	}
}

void FShaderCompileUtilities::DeleteFileHelper(const FString& Filename)
{
	// TODO: This logic came from FShaderCompileThreadRunnable::WriteNewTasks().
	// We can't avoid code duplication unless we refactored the local worker too.

	if (FPlatformFileManager::Get().GetPlatformFile().FileExists(*Filename))
	{
		bool bDeletedOutput = IFileManager::Get().Delete(*Filename, true, true);

		// Retry over the next two seconds if we couldn't delete it
		int32 RetryCount = 0;
		while (!bDeletedOutput && RetryCount < 200)
		{
			FPlatformProcess::Sleep(0.01f);
			bDeletedOutput = IFileManager::Get().Delete(*Filename, true, true);
			RetryCount++;
		}
		checkf(bDeletedOutput, TEXT("Failed to delete %s!"), *Filename);
	}
}


