// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	ShaderGenerationUtil.cpp: Shader generation utilities.
=============================================================================*/

#include "ShaderCompiler.h"

#include "DataDrivenShaderPlatformInfo.h"
#include "HAL/PlatformFile.h"
#include "HAL/PlatformFileManager.h"
#include "Interfaces/ITargetPlatform.h"
#include "Interfaces/ITargetPlatformManagerModule.h"
#include "Misc/FileHelper.h"
#include "RenderUtils.h"
#include "SceneManagement.h"
#include "Serialization/ShaderKeyGenerator.h"
#include "ShaderCompilerDefinitions.h"
#include "ShaderMaterial.h"

static TAutoConsoleVariable<int32> CVarShaderUseGBufferRefactor(
	TEXT("r.Shaders.UseGBufferRefactor"),
	0,
	TEXT("Whether to use the refactored GBuffer that autogenerates encode/decode functions. Will be removed before UE5 ships."),
	ECVF_Default);



#define SET_COMPILE_BOOL_FORCE(X) OutEnvironment.SetDefine(TEXT(#X),SrcDefines.X ? TEXT("1") : TEXT("0"))


bool NeedsVelocityDepth(EShaderPlatform TargetPlatform)
{
	return (DoesProjectSupportDistanceFields() && FDataDrivenShaderPlatformInfo::GetSupportsLumenGI(TargetPlatform))
		|| FDataDrivenShaderPlatformInfo::GetSupportsRayTracing(TargetPlatform);
}

#if WITH_EDITOR

#define SET_COMPILE_BOOL_IF_TRUE(X) { if (DerivedDefines.X) { OutEnvironment.SetDefine(TEXT(#X), 1); } }

#define FETCH_COMPILE_BOOL(X) {																\
	static FShaderCompilerDefineNameCache Cache_##X(TEXT(#X));								\
	SrcDefines.X = Container.GetIntegerValue(Cache_##X, SrcDefines.X) != 0 ? 1 : 0; }

#define FETCH_COMPILE_INT(X) {																\
	static FShaderCompilerDefineNameCache Cache_##X(TEXT(#X));								\
	SrcDefines.X = Container.GetIntegerValue(Cache_##X, SrcDefines.X); }


template<typename ContainerType>
void ApplyFetchEnvironmentInternal(FShaderGlobalDefines& SrcDefines, const ContainerType& Container)
{
	FETCH_COMPILE_BOOL(USES_BASE_PASS_VELOCITY);
	FETCH_COMPILE_BOOL(GBUFFER_HAS_VELOCITY);
	FETCH_COMPILE_BOOL(GBUFFER_HAS_TANGENT);
	FETCH_COMPILE_BOOL(ALLOW_STATIC_LIGHTING);
	FETCH_COMPILE_BOOL(CLEAR_COAT_BOTTOM_NORMAL);
	FETCH_COMPILE_BOOL(IRIS_NORMAL);
	FETCH_COMPILE_BOOL(DXT5_NORMALMAPS);
	FETCH_COMPILE_BOOL(SELECTIVE_BASEPASS_OUTPUTS);
	FETCH_COMPILE_BOOL(USE_DBUFFER);
	FETCH_COMPILE_BOOL(FORWARD_SHADING);
	FETCH_COMPILE_BOOL(PROJECT_VERTEX_FOGGING_FOR_OPAQUE);
	FETCH_COMPILE_BOOL(PROJECT_MOBILE_DISABLE_VERTEX_FOG);
	FETCH_COMPILE_BOOL(PROJECT_ALLOW_GLOBAL_CLIP_PLANE);
	FETCH_COMPILE_BOOL(EARLY_Z_PASS_ONLY_MATERIAL_MASKING);
	FETCH_COMPILE_BOOL(PROJECT_SUPPORT_SKY_ATMOSPHERE);
	FETCH_COMPILE_BOOL(PROJECT_SUPPORT_SKY_ATMOSPHERE_AFFECTS_HEIGHFOG);
	FETCH_COMPILE_BOOL(PROJECT_EXPFOG_MATCHES_VFOG);
	FETCH_COMPILE_BOOL(SUPPORT_CLOUD_SHADOW_ON_FORWARD_LIT_TRANSLUCENT);
	FETCH_COMPILE_BOOL(SUPPORT_CLOUD_SHADOW_ON_SINGLE_LAYER_WATER);
	FETCH_COMPILE_BOOL(SUPPORT_PRIMITIVE_ALPHA_HOLDOUT);
	FETCH_COMPILE_BOOL(PLATFORM_SUPPORTS_SHADER_ROOT_CONSTANTS);
	FETCH_COMPILE_BOOL(PLATFORM_SUPPORTS_SHADER_BUNDLE_DISPATCH);
	FETCH_COMPILE_BOOL(PLATFORM_SUPPORTS_RENDERTARGET_WRITE_MASK);
	FETCH_COMPILE_BOOL(PLATFORM_SUPPORTS_PER_PIXEL_DBUFFER_MASK);
	FETCH_COMPILE_BOOL(PLATFORM_SUPPORTS_DISTANCE_FIELDS);
	FETCH_COMPILE_BOOL(COMPILE_SHADERS_FOR_DEVELOPMENT_ALLOWED);
	FETCH_COMPILE_BOOL(PLATFORM_ALLOW_SCENE_DATA_COMPRESSED_TRANSFORMS);
}

template<typename ContainerType>
void ApplyFetchEnvironmentInternal(FShaderLightmapPropertyDefines& SrcDefines, const ContainerType& Container)
{
	FETCH_COMPILE_BOOL(LQ_TEXTURE_LIGHTMAP);
	FETCH_COMPILE_BOOL(HQ_TEXTURE_LIGHTMAP);

	FETCH_COMPILE_BOOL(CACHED_POINT_INDIRECT_LIGHTING);

	FETCH_COMPILE_BOOL(STATICLIGHTING_TEXTUREMASK);
	FETCH_COMPILE_BOOL(STATICLIGHTING_SIGNEDDISTANCEFIELD);

	FETCH_COMPILE_BOOL(TRANSLUCENT_SELF_SHADOWING);

	FETCH_COMPILE_BOOL(PRECOMPUTED_IRRADIANCE_VOLUME_LIGHTING);
	FETCH_COMPILE_BOOL(CACHED_VOLUME_INDIRECT_LIGHTING);

	FETCH_COMPILE_BOOL(WATER_MESH_FACTORY);
	FETCH_COMPILE_BOOL(NIAGARA_MESH_FACTORY);
	FETCH_COMPILE_BOOL(NIAGARA_MESH_INSTANCED);


	FETCH_COMPILE_BOOL(PARTICLE_MESH_FACTORY);
	FETCH_COMPILE_BOOL(PARTICLE_MESH_INSTANCED);

	FETCH_COMPILE_BOOL(MANUAL_VERTEX_FETCH);

}

template<typename ContainerType>
void ApplyFetchEnvironmentInternal(FShaderMaterialPropertyDefines& SrcDefines, const ContainerType& Container)
{
	FETCH_COMPILE_BOOL(MATERIAL_ENABLE_TRANSLUCENCY_FOGGING);
	FETCH_COMPILE_BOOL(MATERIALBLENDING_ANY_TRANSLUCENT);
	FETCH_COMPILE_BOOL(MATERIAL_USES_SCENE_COLOR_COPY);
	FETCH_COMPILE_BOOL(MATERIALBLENDING_MASKED_USING_COVERAGE);

	FETCH_COMPILE_BOOL(MATERIAL_COMPUTE_FOG_PER_PIXEL);
	FETCH_COMPILE_BOOL(MATERIAL_SHADINGMODEL_UNLIT);

	FETCH_COMPILE_BOOL(MATERIAL_SHADINGMODEL_DEFAULT_LIT);
	FETCH_COMPILE_BOOL(MATERIAL_SHADINGMODEL_SUBSURFACE);
	FETCH_COMPILE_BOOL(MATERIAL_SHADINGMODEL_PREINTEGRATED_SKIN);
	FETCH_COMPILE_BOOL(MATERIAL_SHADINGMODEL_SUBSURFACE_PROFILE);
	FETCH_COMPILE_BOOL(MATERIAL_SHADINGMODEL_CLEAR_COAT);
	FETCH_COMPILE_BOOL(MATERIAL_SHADINGMODEL_TWOSIDED_FOLIAGE);
	FETCH_COMPILE_BOOL(MATERIAL_SHADINGMODEL_HAIR);
	FETCH_COMPILE_BOOL(MATERIAL_SHADINGMODEL_CLOTH);
	FETCH_COMPILE_BOOL(MATERIAL_SHADINGMODEL_EYE);
	FETCH_COMPILE_BOOL(MATERIAL_SHADINGMODEL_SINGLELAYERWATER);
	FETCH_COMPILE_BOOL(MATERIAL_SHADINGMODEL_THIN_TRANSLUCENT);
	FETCH_COMPILE_BOOL(MATERIAL_SHADINGMODEL_TOON);

	FETCH_COMPILE_BOOL(SINGLE_LAYER_WATER_SEPARATED_MAIN_LIGHT);

	FETCH_COMPILE_BOOL(MATERIAL_FULLY_ROUGH);

	FETCH_COMPILE_BOOL(USES_EMISSIVE_COLOR);

	FETCH_COMPILE_BOOL(MATERIALBLENDING_SOLID);
	FETCH_COMPILE_BOOL(MATERIALBLENDING_MASKED);
	FETCH_COMPILE_BOOL(MATERIALBLENDING_ALPHACOMPOSITE);
	FETCH_COMPILE_BOOL(MATERIALBLENDING_TRANSLUCENT);
	FETCH_COMPILE_BOOL(MATERIALBLENDING_ADDITIVE);
	FETCH_COMPILE_BOOL(MATERIALBLENDING_MODULATE);
	FETCH_COMPILE_BOOL(MATERIALBLENDING_ALPHAHOLDOUT);

	FETCH_COMPILE_BOOL(SUBSTRATE_BLENDING_OPAQUE);
	FETCH_COMPILE_BOOL(SUBSTRATE_BLENDING_MASKED);
	FETCH_COMPILE_BOOL(SUBSTRATE_BLENDING_TRANSLUCENT_GREYTRANSMITTANCE);
	FETCH_COMPILE_BOOL(SUBSTRATE_BLENDING_TRANSLUCENT_COLOREDTRANSMITTANCE);
	FETCH_COMPILE_BOOL(SUBSTRATE_BLENDING_COLOREDTRANSMITTANCEONLY);
	FETCH_COMPILE_BOOL(SUBSTRATE_BLENDING_ALPHAHOLDOUT);

	FETCH_COMPILE_INT(MATERIALDECALRESPONSEMASK);

	FETCH_COMPILE_BOOL(REFRACTION_USE_INDEX_OF_REFRACTION);
	FETCH_COMPILE_BOOL(REFRACTION_USE_PIXEL_NORMAL_OFFSET);
	FETCH_COMPILE_BOOL(REFRACTION_USE_2D_OFFSET);

	FETCH_COMPILE_BOOL(USE_DITHERED_LOD_TRANSITION_FROM_MATERIAL);
	FETCH_COMPILE_BOOL(MATERIAL_TWOSIDED);
	FETCH_COMPILE_BOOL(MATERIAL_ISTHINSURFACE);
	FETCH_COMPILE_BOOL(MATERIAL_TANGENTSPACENORMAL);
	FETCH_COMPILE_BOOL(GENERATE_SPHERICAL_PARTICLE_NORMALS);
	FETCH_COMPILE_BOOL(MATERIAL_USE_PREINTEGRATED_GF);
	FETCH_COMPILE_BOOL(MATERIAL_HQ_FORWARD_REFLECTIONS);
	FETCH_COMPILE_BOOL(MATERIAL_PLANAR_FORWARD_REFLECTIONS);
	FETCH_COMPILE_BOOL(MATERIAL_NONMETAL);
	FETCH_COMPILE_BOOL(MATERIAL_USE_LM_DIRECTIONALITY);
	FETCH_COMPILE_BOOL(MATERIAL_INJECT_EMISSIVE_INTO_LPV);
	FETCH_COMPILE_BOOL(MATERIAL_SSR);
	FETCH_COMPILE_BOOL(MATERIAL_CONTACT_SHADOWS);
	FETCH_COMPILE_BOOL(MATERIAL_BLOCK_GI);
	FETCH_COMPILE_BOOL(MATERIAL_DITHER_OPACITY_MASK);
	FETCH_COMPILE_BOOL(MATERIAL_NORMAL_CURVATURE_TO_ROUGHNESS);
	FETCH_COMPILE_BOOL(MATERIAL_ALLOW_NEGATIVE_EMISSIVECOLOR);
	FETCH_COMPILE_BOOL(MATERIAL_OUTPUT_OPACITY_AS_ALPHA);
	FETCH_COMPILE_BOOL(TRANSLUCENT_SHADOW_WITH_MASKED_OPACITY);

	FETCH_COMPILE_BOOL(MATERIAL_DOMAIN_SURFACE);
	FETCH_COMPILE_BOOL(MATERIAL_DOMAIN_DEFERREDDECAL);
	FETCH_COMPILE_BOOL(MATERIAL_DOMAIN_LIGHTFUNCTION);
	FETCH_COMPILE_BOOL(MATERIAL_DOMAIN_VOLUME);
	FETCH_COMPILE_BOOL(MATERIAL_DOMAIN_POSTPROCESS);
	FETCH_COMPILE_BOOL(MATERIAL_DOMAIN_UI);
	FETCH_COMPILE_BOOL(MATERIAL_DOMAIN_VIRTUALTEXTURE);

	FETCH_COMPILE_BOOL(TRANSLUCENCY_LIGHTING_VOLUMETRIC_NONDIRECTIONAL);
	FETCH_COMPILE_BOOL(TRANSLUCENCY_LIGHTING_VOLUMETRIC_DIRECTIONAL);
	FETCH_COMPILE_BOOL(TRANSLUCENCY_LIGHTING_VOLUMETRIC_PERVERTEX_NONDIRECTIONAL);
	FETCH_COMPILE_BOOL(TRANSLUCENCY_LIGHTING_VOLUMETRIC_PERVERTEX_DIRECTIONAL);
	FETCH_COMPILE_BOOL(TRANSLUCENCY_LIGHTING_SURFACE_LIGHTINGVOLUME);
	FETCH_COMPILE_BOOL(TRANSLUCENCY_LIGHTING_SURFACE_FORWARDSHADING);

	FETCH_COMPILE_BOOL(EDITOR_PRIMITIVE_MATERIAL);

	FETCH_COMPILE_BOOL(USE_STENCIL_LOD_DITHER_DEFAULT);

	FETCH_COMPILE_BOOL(MATERIALDOMAIN_SURFACE);
	FETCH_COMPILE_BOOL(MATERIALDOMAIN_DEFERREDDECAL);
	FETCH_COMPILE_BOOL(MATERIALDOMAIN_LIGHTFUNCTION);
	FETCH_COMPILE_BOOL(MATERIALDOMAIN_POSTPROCESS);
	FETCH_COMPILE_BOOL(MATERIALDOMAIN_UI);

	FETCH_COMPILE_BOOL(OUT_BASECOLOR);
	FETCH_COMPILE_BOOL(OUT_BASECOLOR_NORMAL_ROUGHNESS);
	FETCH_COMPILE_BOOL(OUT_BASECOLOR_NORMAL_SPECULAR);
	FETCH_COMPILE_BOOL(OUT_MASK4);
	FETCH_COMPILE_BOOL(OUT_WORLDHEIGHT);
	FETCH_COMPILE_BOOL(OUT_DISPLACEMENT);

	FETCH_COMPILE_BOOL(IS_VIRTUAL_TEXTURE_MATERIAL);
	FETCH_COMPILE_BOOL(IS_DECAL);
	FETCH_COMPILE_BOOL(IS_BASE_PASS);
	FETCH_COMPILE_BOOL(COMPUTE_SHADED);
	FETCH_COMPILE_BOOL(USES_WORLD_POSITION_OFFSET);
	FETCH_COMPILE_BOOL(USES_FIRST_PERSON_INTERPOLATION);
	FETCH_COMPILE_BOOL(IS_MATERIAL_SHADER);

	FETCH_COMPILE_BOOL(SUBSTRATE_ENABLED);
	FETCH_COMPILE_INT(SUBSTRATE_GBUFFER_FORMAT);
	FETCH_COMPILE_BOOL(MATERIAL_IS_SUBSTRATE);

	FETCH_COMPILE_BOOL(PROJECT_OIT);

	FETCH_COMPILE_BOOL(DUAL_SOURCE_COLOR_BLENDING_ENABLED);

	FETCH_COMPILE_INT(DECAL_RENDERTARGET_COUNT);

	FETCH_COMPILE_INT(GBUFFER_LAYOUT);
}

template<typename ContainerType>
void ApplyFetchEnvironmentInternal(FShaderCompilerDefines& SrcDefines, const ContainerType& Container)
{
	FETCH_COMPILE_BOOL(COMPILER_GLSL_ES3_1);
	FETCH_COMPILE_BOOL(ES3_1_PROFILE);

	FETCH_COMPILE_BOOL(COMPILER_GLSL);
	FETCH_COMPILE_BOOL(COMPILER_GLSL_ES3_1_EXT);
	FETCH_COMPILE_BOOL(ESDEFERRED_PROFILE);
	FETCH_COMPILE_BOOL(GL4_PROFILE);

	FETCH_COMPILE_BOOL(METAL_PROFILE);
	FETCH_COMPILE_BOOL(VULKAN_PROFILE);
	FETCH_COMPILE_BOOL(MAC);

	FETCH_COMPILE_BOOL(PLATFORM_SUPPORTS_DEVELOPMENT_SHADERS);
}

void FShaderCompileUtilities::ApplyFetchEnvironment(FShaderGlobalDefines& SrcDefines, const FShaderCompilerEnvironment& Environment, const EShaderPlatform Platform)
{
	ApplyFetchEnvironmentInternal(SrcDefines, *Environment.Definitions.Get());

	// note that we are doing an if so that if we call ApplyFetchEnvironment() twice, we get the logical OR of bSupportsDualBlending support
	if (RHISupportsDualSourceBlending(Platform))
	{
		SrcDefines.bSupportsDualBlending = true;
	}
}
void FShaderCompileUtilities::ApplyFetchEnvironment(FShaderLightmapPropertyDefines& SrcDefines, const FShaderCompilerEnvironment& Environment)
{
	ApplyFetchEnvironmentInternal(SrcDefines, *Environment.Definitions.Get());
}
void FShaderCompileUtilities::ApplyFetchEnvironment(FShaderMaterialPropertyDefines& SrcDefines, const FShaderCompilerEnvironment& Environment)
{
	ApplyFetchEnvironmentInternal(SrcDefines, *Environment.Definitions.Get());
}
void FShaderCompileUtilities::ApplyFetchEnvironment(FShaderCompilerDefines& SrcDefines, const FShaderCompilerEnvironment& Environment)
{
	ApplyFetchEnvironmentInternal(SrcDefines, *Environment.Definitions.Get());
}

/** Used to gather names of defines used in the ApplyFetchEnvironmentInternal functions */
struct FDefineNameGather
{
	mutable TArray<FName> Names;

	FORCEINLINE int32 GetIntegerValue(FShaderCompilerDefineNameCache& Cache, int32 DefaultValue) const
	{
		Names.Add(Cache);
		return 0;
	}
};

/** Singleton to initialize initial defines at startup */
struct FShaderInitialDefinesInitializer
{
	FShaderInitialDefinesInitializer()
	{
		FShaderMaterialPropertyDefines MaterialDefines = {};
		FShaderLightmapPropertyDefines LightmapDefines = {};
		FShaderGlobalDefines GlobalDefines = {};
		FShaderCompilerDefines CompilerDefines = {};

		FDefineNameGather GatherNames;

		ApplyFetchEnvironmentInternal(GlobalDefines, GatherNames);
		ApplyFetchEnvironmentInternal(MaterialDefines, GatherNames);
		ApplyFetchEnvironmentInternal(LightmapDefines, GatherNames);
		ApplyFetchEnvironmentInternal(CompilerDefines, GatherNames);

		FShaderCompilerDefinitions InitialDefines;
		for (FName DefineKey : GatherNames.Names)
		{
			// Add a key, leaving the value unset (EShaderCompilerDefineVariant::None)
			InitialDefines.FindOrAddMapIndex(DefineKey);
		}
		FShaderCompilerDefinitions::InitializeInitialDefines(InitialDefines);
	}
};
static FShaderInitialDefinesInitializer GInitialDefinesInitializer;


// if we change the logic, increment this number to force a DDC key change
static const int32 GBufferGeneratorVersion = 5;

static FShaderGlobalDefines FetchShaderGlobalDefines(EShaderPlatform TargetPlatform, EGBufferLayout GBufferLayout)
{
	FShaderGlobalDefines Ret = {};

	bool bIsMobilePlatform = IsMobilePlatform(TargetPlatform);

	Ret.ALLOW_STATIC_LIGHTING = IsStaticLightingAllowed();
	Ret.USES_BASE_PASS_VELOCITY = IsUsingBasePassVelocity(TargetPlatform);
	Ret.GBUFFER_HAS_VELOCITY = (Ret.USES_BASE_PASS_VELOCITY || GBufferLayout == GBL_ForceVelocity) ? 1 : 0;
	Ret.GBUFFER_HAS_TANGENT = false;//BasePassCanOutputTangent(TargetPlatform) ? 1 : 0;

	{
		static const auto CVar = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.ClearCoatNormal"));
		Ret.CLEAR_COAT_BOTTOM_NORMAL = CVar ? (CVar->GetValueOnAnyThread() != 0 && (!bIsMobilePlatform || !IsMobileDeferredShadingEnabled(TargetPlatform) || MobileUsesExtendedGBuffer(TargetPlatform))) : 0;
	}

	{
		static const auto CVar = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.IrisNormal"));
		Ret.IRIS_NORMAL = CVar ? (CVar->GetValueOnAnyThread() != 0) : 0;
	}

	{
		static const auto CVar = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("Compat.UseDXT5NormalMaps"));
		Ret.DXT5_NORMALMAPS = CVar ? (CVar->GetValueOnAnyThread() != 0) : 0;
	}

	Ret.SELECTIVE_BASEPASS_OUTPUTS = IsUsingSelectiveBasePassOutputs((EShaderPlatform)TargetPlatform) ? 1 : 0;
	Ret.USE_DBUFFER = IsUsingDBuffers((EShaderPlatform)TargetPlatform) ? 1 : 0;

#if WITH_EDITOR
	{
		bool bForwardShading = false;

		ITargetPlatform* TargetPlatformPtr = GetTargetPlatformManager()->FindTargetPlatformWithSupport(TEXT("ShaderFormat"), LegacyShaderPlatformToShaderFormat((EShaderPlatform)TargetPlatform));
		if (TargetPlatformPtr)
		{
			bForwardShading = TargetPlatformPtr->UsesForwardShading();
		}
		else
		{
			static IConsoleVariable* CVarForwardShading = IConsoleManager::Get().FindConsoleVariable(TEXT("r.ForwardShading"));
			bForwardShading = CVarForwardShading ? (CVarForwardShading->GetInt() != 0) : false;
		}
		Ret.FORWARD_SHADING = bForwardShading;
	}
#else
	{
		static const auto CVar = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.ForwardShading"));
		Ret.FORWARD_SHADING = CVar ? (CVar->GetValueOnAnyThread() != 0) : 0;
	}
#endif

	if (bIsMobilePlatform)
	{
		Ret.FORWARD_SHADING = !IsMobileDeferredShadingEnabled((EShaderPlatform)TargetPlatform);
	}

	{
		static IConsoleVariable* CVar = IConsoleManager::Get().FindConsoleVariable(TEXT("r.VertexFoggingForOpaque"));
		Ret.PROJECT_VERTEX_FOGGING_FOR_OPAQUE = Ret.FORWARD_SHADING && (CVar ? (CVar->GetInt() != 0) : 0);
	}

	{
		static IConsoleVariable* CVar = IConsoleManager::Get().FindConsoleVariable(TEXT("r.Mobile.DisableVertexFog"));
		Ret.PROJECT_MOBILE_DISABLE_VERTEX_FOG = CVar ? (CVar->GetInt() != 0) : 0;
	}

	{
		static IConsoleVariable* CVar = IConsoleManager::Get().FindConsoleVariable(TEXT("r.AllowGlobalClipPlane"));
		Ret.PROJECT_ALLOW_GLOBAL_CLIP_PLANE = CVar ? (CVar->GetInt() != 0) : 0;
	}

	{
		if (MaskedInEarlyPass((EShaderPlatform)TargetPlatform))
		{
			Ret.EARLY_Z_PASS_ONLY_MATERIAL_MASKING = 1;
		}
		else
		{
			Ret.EARLY_Z_PASS_ONLY_MATERIAL_MASKING = 0;
		}
	}

	bool bSupportSkyAtmosphere = false;
	{
		static IConsoleVariable* CVar = IConsoleManager::Get().FindConsoleVariable(TEXT("r.SupportSkyAtmosphere"));
		bSupportSkyAtmosphere = CVar && CVar->GetInt() != 0;
		Ret.PROJECT_SUPPORT_SKY_ATMOSPHERE = bSupportSkyAtmosphere ? 1 : 0;
	}

	{
		static IConsoleVariable* CVar = IConsoleManager::Get().FindConsoleVariable(TEXT("r.SupportSkyAtmosphereAffectsHeightFog"));
		Ret.PROJECT_SUPPORT_SKY_ATMOSPHERE_AFFECTS_HEIGHFOG = (CVar && bSupportSkyAtmosphere) ? (CVar->GetInt() != 0) : 0;
	}

	{
		Ret.PROJECT_EXPFOG_MATCHES_VFOG = DoesProjectSupportExpFogMatchesVolumetricFog() ? 1 : 0;
	}

	{
		static IConsoleVariable* CVar = IConsoleManager::Get().FindConsoleVariable(TEXT("r.SupportCloudShadowOnForwardLitTranslucent"));
		const bool bSupportCloudShadowOnForwardLitTranslucent = CVar && CVar->GetInt() > 0;
		Ret.SUPPORT_CLOUD_SHADOW_ON_FORWARD_LIT_TRANSLUCENT = bSupportCloudShadowOnForwardLitTranslucent ? 1 : 0;
	}

	{
		static IConsoleVariable *CVar = IConsoleManager::Get().FindConsoleVariable(TEXT("r.Water.SingleLayerWater.SupportCloudShadow"));
		const bool bSupportCloudShadowOnSingleLayerWater = CVar && CVar->GetInt() > 0;
		Ret.SUPPORT_CLOUD_SHADOW_ON_SINGLE_LAYER_WATER = bSupportCloudShadowOnSingleLayerWater ? 1 : 0;
	}

	{
		static IConsoleVariable* CVar = IConsoleManager::Get().FindConsoleVariable(TEXT("r.Deferred.SupportPrimitiveAlphaHoldout"));
		Ret.SUPPORT_PRIMITIVE_ALPHA_HOLDOUT = CVar->GetBool();
	}

	Ret.PLATFORM_SUPPORTS_SHADER_ROOT_CONSTANTS = RHISupportsShaderRootConstants(EShaderPlatform(TargetPlatform)) ? 1 : 0;
	Ret.PLATFORM_SUPPORTS_SHADER_BUNDLE_DISPATCH = RHISupportsShaderBundleDispatch(EShaderPlatform(TargetPlatform)) ? 1 : 0;
	Ret.PLATFORM_SUPPORTS_RENDERTARGET_WRITE_MASK = RHISupportsRenderTargetWriteMask(EShaderPlatform(TargetPlatform)) ? 1 : 0;
	Ret.PLATFORM_SUPPORTS_PER_PIXEL_DBUFFER_MASK = FDataDrivenShaderPlatformInfo::GetSupportsPerPixelDBufferMask(EShaderPlatform(TargetPlatform)) ? 1 : 0;
	Ret.PLATFORM_SUPPORTS_DISTANCE_FIELDS = DoesPlatformSupportDistanceFields(EShaderPlatform(TargetPlatform)) ? 1 : 0;
	Ret.PLATFORM_ALLOW_SCENE_DATA_COMPRESSED_TRANSFORMS = FDataDrivenShaderPlatformInfo::GetSupportSceneDataCompressedTransforms(EShaderPlatform(TargetPlatform)) ? 1 : 0;

	Ret.bSupportsDualBlending = RHISupportsDualSourceBlending(TargetPlatform);

	return Ret;
}

static FString GetSlotTextName(EGBufferSlot Slot)
{
	FString Ret;
	switch (Slot)
	{
	case GBS_Invalid:
		check(0);
		return TEXT("Invalid");
	case GBS_SceneColor:
		return TEXT("SceneColor");
	case GBS_WorldNormal:
		return TEXT("WorldNormal");
	case GBS_PerObjectGBufferData:
		return TEXT("PerObjectGBufferData");
	case GBS_Metallic:
		return TEXT("Metallic");
	case GBS_Specular:
		return TEXT("Specular");
	case GBS_Roughness:
		return TEXT("Roughness");
	case GBS_ShadingModelId:
		return TEXT("ShadingModelID");
	case GBS_SelectiveOutputMask:
		return TEXT("SelectiveOutputMask");
	case GBS_BaseColor:
		return TEXT("BaseColor");
	case GBS_GenericAO:
		return TEXT("GenericAO");
	case GBS_IndirectIrradiance:
		return TEXT("IndirectIrradiance");
	case GBS_Velocity:
		return TEXT("Velocity");
	case GBS_PrecomputedShadowFactor:
		return TEXT("PrecomputedShadowFactors");
	case GBS_WorldTangent:
		return TEXT("WorldTangent");
	case GBS_Anisotropy:
		return TEXT("Anisotropy");
	case GBS_CustomData:
		return TEXT("CustomData");
	case GBS_SubsurfaceColor:
		return TEXT("SubsurfaceColor");
	case GBS_Opacity:
		return TEXT("Opacity");
	case GBS_SubsurfaceProfile:
		return TEXT("SubsurfaceProfile");
	case GBS_ClearCoat:
		return TEXT("ClearCoat");
	case GBS_ClearCoatRoughness:
		return TEXT("ClearCoatRoughness");
	case GBS_HairSecondaryWorldNormal:
		return TEXT("HaireEcondaryWorldNormal");
	case GBS_HairBacklit:
		return TEXT("HairBacklit");
	case GBS_Cloth:
		return TEXT("Cloth");
	case GBS_SubsurfaceProfileX:
		return TEXT("SubsurfaceProfileX");
	case GBS_IrisNormal:
		return TEXT("IrisNormal");
	case GBS_SeparatedMainDirLight:
		return TEXT("SeparatedMainDirLight");
	default:
		break;
	};

	return TEXT("ERROR");
}


static FString GetFloatType(int32 ChanNum)
{
	switch (ChanNum)
	{
	case 0:
		check(0);
		return TEXT("Error");
	case 1:
		return TEXT("float");
	case 2:
		return TEXT("float2");
	case 3:
		return TEXT("float3");
	case 4:
		return TEXT("float4");
	default:
		break;
	}

	check(0);
	return TEXT("Error");
}

static FString GetUintType(int32 ChanNum)
{
	switch (ChanNum)
	{
	case 0:
		check(0);
		return TEXT("Error");
	case 1:
		return TEXT("uint");
	case 2:
		return TEXT("uint2");
	case 3:
		return TEXT("uint3");
	case 4:
		return TEXT("uint4");
	default:
		break;
	}

	check(0);
	return TEXT("Error");
}


static const FGBufferCompressionInfo GBufferCompressionInfo[] =
{
	{ GBC_Invalid,					    0, 0, {  0,  0,  0,  0 }, false, false, TEXT("Invalid")                , TEXT("Invalid")                },
	{ GBC_Raw_Float_16_16_16_16,		4, 4, { 16, 16, 16, 16 }, false, false, TEXT("Invalid")                , TEXT("Invalid")                },
	{ GBC_Raw_Float_16_16_16,			3, 3, { 16, 16, 16,  0 }, false, false, TEXT("Invalid")                , TEXT("Invalid")                },
	{ GBC_Raw_Float_11_11_10,			3, 3, { 11, 11, 10,  0 }, false, false, TEXT("Invalid")                , TEXT("Invalid")                },
	{ GBC_Raw_Float_10_10_10,			3, 3, { 10, 10, 10,  0 }, false, false, TEXT("Invalid")                , TEXT("Invalid")                },
	{ GBC_Raw_Unorm_8_8_8_8,			4, 4, {  8,  8,  8,  8 }, false, false, TEXT("Invalid")                , TEXT("Invalid")                },
	{ GBC_Raw_Unorm_8_8_8,			  	3, 3, {  8,  8,  8,  0 }, false, false, TEXT("Invalid")                , TEXT("Invalid")                },
	{ GBC_Raw_Unorm_8_8,				2, 2, {  8,  8,  0,  0 }, false, false, TEXT("Invalid")                , TEXT("Invalid")                },
	{ GBC_Raw_Unorm_8,				  	1, 1, {  8,  0,  0,  0 }, false, false, TEXT("Invalid")                , TEXT("Invalid")                },
	{ GBC_Raw_Unorm_2, 				  	1, 1, {  2,  0,  0,  0 }, false, false, TEXT("Invalid")                , TEXT("Invalid")                },
	{ GBC_Raw_Float_16_16,			  	2, 2, { 16, 16,  0,  0 }, false, false, TEXT("Invalid")                , TEXT("Invalid")                },
	{ GBC_Raw_Float_16,				  	1, 1, { 16,  0,  0,  0 }, false, false, TEXT("Invalid")                , TEXT("Invalid")                },
	{ GBC_Bits_4, 					  	1, 1, {  4,  0,  0,  0 },  true, false, TEXT("Invalid")                , TEXT("Invalid")                },
	{ GBC_Bits_2, 					  	1, 1, {  2,  0,  0,  0 },  true, false, TEXT("Invalid")                , TEXT("Invalid")                },
	{ GBC_Packed_Normal_Octahedral_8_8, 3, 2, {  8,  8,  0,  0 }, false,  true, TEXT("CompressOctahedral")     , TEXT("DecompressOctahedral")   },
	{ GBC_EncodeNormal_Normal_16_16_16, 3, 3, { 16, 16, 16,  0 }, false,  true, TEXT("EncodeNormalHelper")     , TEXT("DecodeNormalHelper")     },
	{ GBC_EncodeNormal_Normal_10_10_10, 3, 3, { 10, 10, 10,  0 }, false,  true, TEXT("EncodeNormalHelper")     , TEXT("DecodeNormalHelper")     },
	{ GBC_EncodeNormal_Normal_8_8_8,    3, 3, {  8,  8,  8,  0 }, false,  true, TEXT("EncodeNormalHelper")     , TEXT("DecodeNormalHelper")     },
	{ GBC_Packed_Color_5_6_5,			3, 3, {  5,  6,  5,  0 },  true,  true, TEXT("EncodeQuantize565")      , TEXT("DecodeQuantize565")      },
	{ GBC_Packed_Color_5_6_5_Sqrt,		3, 3, {  5,  6,  5,  0 },  true,  true, TEXT("EncodeQuantize565Sqrt")  , TEXT("DecodeQuantize565Sqrt")  },
	{ GBC_Packed_Color_4_4_4,			3, 3, {  4,  4,  4,  0 },  true,  true, TEXT("EncodeQuantize444")      , TEXT("DecodeQuantize444")	    },
	{ GBC_Packed_Color_4_4_4_Sqrt,		3, 3, {  4,  4,  4,  0 },  true,  true, TEXT("EncodeQuantize444Sqrt")  , TEXT("DecodeQuantize444Sqrt")  },
	{ GBC_Packed_Color_3_3_2,			3, 3, {  3,  3,  2,  0 },  true,  true, TEXT("EncodeQuantize332")      , TEXT("DecodeQuantize332")      },
	{ GBC_Packed_Color_3_3_2_Sqrt,		3, 3, {  3,  3,  2,  0 },  true,  true, TEXT("EncodeQuantize332Sqrt")  , TEXT("DecodeQuantize332Sqrt")  },
	{ GBC_Packed_Quantized_6,			1, 1, {  6,  0,  0,  0 },  true,  true, TEXT("EncodeQuantize6")        , TEXT("DecodeQuantize6")        },
	{ GBC_Packed_Quantized_4,			1, 1, {  4,  0,  0,  0 },  true,  true, TEXT("EncodeQuantize4")        , TEXT("DecodeQuantize4")        },
	{ GBC_Packed_Quantized_2,			1, 1, {  2,  0,  0,  0 },  true,  true, TEXT("EncodeQuantize2")        , TEXT("DecodeQuantize2")        },
};

static int32 GetGBufferCompressionBitSize(EGBufferCompression Compression)
{
	check(Compression >= 0 && Compression < GBC_Num);
	check(GBufferCompressionInfo[Compression].Type == Compression);

	int32 BitSize = 0;
	for (int32 I = 0; I < 4; I++)
	{
		BitSize += GBufferCompressionInfo[Compression].ChanBits[I];
	}

	return BitSize;
}

static int32 GetGBufferCompressionChanBitSize(EGBufferCompression Compression, int32 Chan)
{
	check(Compression >= 0 && Compression < GBC_Num);
	check(GBufferCompressionInfo[Compression].Type == Compression);
	check(Chan >= 0 && Chan < 4);

	int32 BitSize = GBufferCompressionInfo[Compression].ChanBits[Chan];

	return BitSize;
}

static bool IsPackedBits(EGBufferCompression Compression)
{
	check(Compression >= 0 && Compression < GBC_Num);
	check(GBufferCompressionInfo[Compression].Type == Compression);
	return GBufferCompressionInfo[Compression].bIsPackedBits;
}

// for each compression type, the number of incoming and outgoing channels. So for example octahedral encoding goes from
// 3 channels to 2, so GetCompressionSrcNumChannels() is 3 and GetCompressionDstNumChannels() is 2.
static int32 GetCompressionSrcNumChannels(EGBufferCompression Compression)
{
	check(Compression >= 0 && Compression < GBC_Num);
	check(GBufferCompressionInfo[Compression].Type == Compression);
	return GBufferCompressionInfo[Compression].SrcNumChan;
}

static int32 GetCompressionDstNumChannels(EGBufferCompression Compression)
{
	check(Compression >= 0 && Compression < GBC_Num);
	check(GBufferCompressionInfo[Compression].Type == Compression);
	return GBufferCompressionInfo[Compression].DstNumChan;
}

// does this type require some kind of pack/unpack step (such as octahedral encoding) other than just bit packing
static int32 GetCompressionNeedsConversion(EGBufferCompression Compression)
{
	check(Compression >= 0 && Compression < GBC_Num);
	check(GBufferCompressionInfo[Compression].Type == Compression);
	return GBufferCompressionInfo[Compression].bIsConversion;
}

// does this type require some kind of pack/unpack step (such as octahedral encoding) other than just bit packing
static FString GetCompressionEncodeFuncName(EGBufferCompression Compression)
{
	check(Compression >= 0 && Compression < GBC_Num);
	check(GBufferCompressionInfo[Compression].Type == Compression);
	return GBufferCompressionInfo[Compression].EncodeFuncName;
}


// does this type require some kind of pack/unpack step (such as octahedral encoding) other than just bit packing
static FString GetCompressionDecodeFuncName(EGBufferCompression Compression)
{
	check(Compression >= 0 && Compression < GBC_Num);
	check(GBufferCompressionInfo[Compression].Type == Compression);
	return GBufferCompressionInfo[Compression].DecodeFuncName;
}

static int32 GetBufferNumBits(EGBufferType Format, int32 Channel)
{
	check(Channel >= 0);
	check(Channel < 4);

	int32 Ret = 0;
	switch (Format)
	{
	case GBT_Invalid:
		check(0);
		break;
	case GBT_Unorm_16_16:
		Ret = 16;
		break;
	case GBT_Unorm_8_8_8_8:
		Ret = 8;
		break;
	case GBT_Unorm_11_11_10:
	{
		int32 Sizes[4] = { 11, 11, 10, 0 };
		Ret = Sizes[Channel];
	}
	break;
	case GBT_Unorm_10_10_10_2:
	{
		int32 Sizes[4] = { 10, 10, 10, 2 };
		Ret = Sizes[Channel];
	}
	break;
	case GBT_Unorm_16_16_16_16:
		Ret = 16;
		break;
	case GBT_Float_16_16:
		Ret = 16;
		break;
	case GBT_Float_16_16_16_16:
		Ret = 16;
		break;
	case GBT_Float_32:
		Ret = 32;
	default:
		check(0);
		break;
	}

	return Ret;
}

static int32 GetTargetNumChannels(EGBufferType Type)
{
	int32 Ret = 0;

	switch (Type)
	{
	case GBT_Invalid:
		Ret = 0;
		break;
	case GBT_Unorm_16_16:
		Ret = 2;
		break;
	case GBT_Unorm_8_8_8_8:
		Ret = 4;
		break;
	case GBT_Unorm_11_11_10:
		Ret = 3;
		break;
	case GBT_Unorm_10_10_10_2:
		Ret = 4;
		break;
	case GBT_Unorm_16_16_16_16:
		Ret = 4;
		break;
	case GBT_Float_16_16:
		Ret = 2;
		break;
	case GBT_Float_16_16_16_16:
		Ret = 4;
		break;
	case GBT_Float_32:
		Ret = 1;
		break;
	default:
		check(0);
		break;
	}

	return Ret;
}


// Quick note: There are lots of instances where we do something like:
//
// CurrLine = SomeComplicatedString();
// FullStr += CurrLine;
//
// We could instead make those run on a single line, as in:
//
// FullStr += SomeComplicatedString();
//
// The problem is it then becomes a nightmare to step through and debug the lines, which is why each line is fully created before appending.

static FString CreateGBufferEncodeFunction(const FGBufferInfo& BufferInfo)
{
	FString FullStr;

	FullStr += TEXT("void EncodeGBufferToMRT(inout FPixelShaderOut Out, FGBufferData GBuffer, float QuantizationBias)\n");
	FullStr += TEXT("{\n");

	int32 TargetChanNum[FGBufferInfo::MaxTargets] = {};

	// for each values, 0 means float, 1 means int, and -1 means unused
	int32 PackedStatus[FGBufferInfo::MaxTargets][4];
	for (int32 I = 0; I < FGBufferInfo::MaxTargets; I++)
	{
		for (int32 J = 0; J < 4; J++)
		{
			PackedStatus[I][J] = -1;
		}
	}

	bool bNeedsConversion[GBS_Num] = {};

	// For each item that we are writing to the gbuffer, check if it is a float or an an int. Mixing and matching
	// floats and ints to the same channel is illegal. In that case, the float needs to be explicitly packed to an int,
	// and then gets merged with the other int.

	for (int32 I = 0; I < GBS_Num; I++)
	{
		const FGBufferItem& Item = BufferInfo.Slots[I];

		if (Item.bIsValid)
		{
			bool isPacked = IsPackedBits(Item.Compression);
			int32 NumComponents = GetCompressionDstNumChannels(Item.Compression);

			for (int32 PackIndex = 0; PackIndex < FGBufferItem::MaxPacking; PackIndex++)
			{
				const FGBufferPacking& PackInfo = Item.Packing[PackIndex];

				if (PackInfo.bIsValid)
				{
					int32 DstTarget = PackInfo.TargetIndex;
					int32 DstChannel = PackInfo.DstChannelIndex;

					if (!isPacked)
					{
						// if it is not packed, make sure the GBuffer is expecting a raw item
						// and make sure we didn't already set it
						check(PackInfo.bFull);
						check(PackedStatus[DstTarget][DstChannel] == -1);
						PackedStatus[DstTarget][DstChannel] = 0;
					}
					else
					{
						// if is packed, make sure the GBuffer is expecting it.
						check(!PackInfo.bFull);

						// since multiple items are packed in a single channel, make sure it's either unset or packed (not zero, which means float)
						check(PackedStatus[DstTarget][DstChannel] == -1 || PackedStatus[DstTarget][DstChannel] == 1);

						PackedStatus[DstTarget][DstChannel] = 1;
					}
				}
			}

			// also, for this slot, check if it needs a type conversion
			bNeedsConversion[I] = GetCompressionNeedsConversion(Item.Compression) != 0;
		}
	}

	// for any parameters that need conversion 

	// write the float and int parameters that we need
	for (int32 I = 0; I < FGBufferInfo::MaxTargets; I++)
	{
		// special case for target 0, for now
		if (I == 0)
		{
			continue;
		}

		int32 ChanNum = GetTargetNumChannels(BufferInfo.Targets[I].TargetType);
		TargetChanNum[I] = ChanNum;

		bool bAnyFloat = false;
		bool bAnyUint = false;

		for (int32 Chan = 0; Chan < 4; Chan++)
		{
			if (PackedStatus[I][Chan] == 0)
			{
				bAnyFloat = true;
			}
			if (PackedStatus[I][Chan] == 1)
			{
				bAnyUint = true;
			}
		}

		if (ChanNum > 0)
		{
			if (bAnyFloat)
			{
				FString FloatName = GetFloatType(ChanNum);
				FullStr += FString::Printf(TEXT("\t%s MrtFloat%d = 0.0f;\n"), FloatName.GetCharArray().GetData(), I);
			}

			if (bAnyUint)
			{
				FString UintName = GetUintType(ChanNum);
				FullStr += FString::Printf(TEXT("\t%s MrtUint%d = 0;\n"), UintName.GetCharArray().GetData(), I);
			}
		}
	}

	FullStr += TEXT("\n");

	int32 NumAdded = 0;
	for (int32 ItemIndex = 0; ItemIndex < GBS_Num; ItemIndex++)
	{
		const FGBufferItem& Item = BufferInfo.Slots[ItemIndex];

		// special case for now
		if (Item.BufferSlot == GBS_SceneColor)
		{
			continue;
		}

		if (Item.bIsValid)
		{
			bool bIsCompressed = GetCompressionNeedsConversion(Item.Compression) != 0;
			if (bIsCompressed)
			{
				bNeedsConversion[ItemIndex] = true;

				// so far we are assuming all input data is float
				FString ConversionName = GetCompressionEncodeFuncName(Item.Compression);
				FString BaseName = GetSlotTextName(Item.BufferSlot);

				int32 SrcChan = GetCompressionDstNumChannels(Item.Compression);

				FString SrcTypeName;
				bool bIsPackedBits = IsPackedBits(Item.Compression);
				if (bIsPackedBits)
				{
					SrcTypeName = GetUintType(SrcChan);
				}
				else
				{
					SrcTypeName = GetFloatType(SrcChan);
				}

				FString QuantizationValue = "0.0f";
				if (Item.bQuantizationBias)
				{
					QuantizationValue = TEXT("QuantizationBias");
				}

				// If there is no quanzitation bias, just add the values
				FString CurrLine = FString::Printf(TEXT("\t%s %s_Compressed = %s(GBuffer.%s, %s);\n"),
					SrcTypeName.GetCharArray().GetData(),
					BaseName.GetCharArray().GetData(),
					ConversionName.GetCharArray().GetData(),
					BaseName.GetCharArray().GetData(),
					QuantizationValue.GetCharArray().GetData());
				FullStr += CurrLine;

				NumAdded++;
			}
		}
	}

	if (NumAdded > 0)
	{
		FullStr += TEXT("\n");
	}

	FString Swizzles[4];
	Swizzles[0] = TEXT("x");
	Swizzles[1] = TEXT("y");
	Swizzles[2] = TEXT("z");
	Swizzles[3] = TEXT("w");

	for (int32 ItemIndex = 0; ItemIndex < GBS_Num; ItemIndex++)
	{
		const FGBufferItem& Item = BufferInfo.Slots[ItemIndex];

		// hack for now
		if (Item.BufferSlot == GBS_SceneColor)
		{
			continue;
		}

		if (Item.bIsValid)
		{
			FString SlotName = GetSlotTextName(Item.BufferSlot);

			FString DataName;
			if (bNeedsConversion[ItemIndex])
			{
				DataName = FString::Printf(TEXT("%s_Compressed"), SlotName.GetCharArray().GetData());
			}
			else
			{
				DataName = FString::Printf(TEXT("GBuffer.%s"), SlotName.GetCharArray().GetData());
			}

/*
			if (Item.BufferSlot == GBS_BaseColor)
			{
				DataName = TEXT("float4(0.25,.25,.25,.25)");
			}
*/
			for (int32 PackIter = 0; PackIter < FGBufferItem::MaxPacking; PackIter++)
			{
				const FGBufferPacking& PackItem = Item.Packing[PackIter];
				if (PackItem.bIsValid)
				{

					FString FullName = FString::Printf(TEXT("%s.%s"),
						DataName.GetCharArray().GetData(),
						Swizzles[PackItem.SrcChannelIndex].GetCharArray().GetData());


					if (PackItem.bFull)
					{
						// if it's a full channel, just pass it in
						FString TargetName = FString::Printf(TEXT("MrtFloat%d"), PackItem.TargetIndex);

						// If we are full, add a quantization term based on the output GBuffer size
						if (Item.bQuantizationBias)
						{
							int DstNumBits = GetGBufferCompressionChanBitSize(Item.Compression,PackItem.DstChannelIndex);

							FullName += FString::Printf(TEXT(" + QuantizationBias / float(%d)"), 1 << DstNumBits);
						}

						// how many bits are we packing?
						FString CurrLine = FString::Printf(TEXT("\t%s.%s = %s;\n"),
							TargetName.GetCharArray().GetData(),
							Swizzles[PackItem.DstChannelIndex].GetCharArray().GetData(),
							FullName.GetCharArray().GetData());

						FullStr += CurrLine;
					}
					else
					{
						check(PackItem.BitNum >= 1);
						check(PackItem.BitNum <= 31);

						int32 Mask = (1 << PackItem.BitNum)-1;

						// if it's a full channel, just pass it in
						FString TargetName = FString::Printf(TEXT("MrtUint%d"), PackItem.TargetIndex);

						FString CurrLine = FString::Printf(TEXT("\t%s.%s |= ((((%s) >> %d) & 0x%02x) << %d);\n"),
							TargetName.GetCharArray().GetData(),
							Swizzles[PackItem.DstChannelIndex].GetCharArray().GetData(),
							FullName.GetCharArray().GetData(),
							PackItem.SrcBitStart,
							Mask,
							PackItem.DstBitStart);

						FullStr += CurrLine;
					}
				}
			}
		}
	}


	FullStr += TEXT("\n");

	for (int32 I = 0; I < FGBufferInfo::MaxTargets; I++)
	{
		// skip first target for now
		if (I == 0)
		{
			continue;
		}

		int32 ChanNum = GetTargetNumChannels(BufferInfo.Targets[I].TargetType);

		// optimization for readability
		bool bIsAnyInt = false;
		bool bIsAnyNonZero = false;
		for (int32 Chan = 0; Chan < ChanNum; Chan++)
		{
			if (PackedStatus[I][Chan] == -1)
			{
				// if it's -1, it's unused (neither int nor float) and just set to 0.0f
			}
			else
			{
				bIsAnyNonZero = true;
				// 0 means float, 1 means int
				if (PackedStatus[I][Chan] == 1)
				{
					bIsAnyInt = true;
				}
			}
		}

		if (bIsAnyNonZero && !bIsAnyInt && ChanNum == 4)
		{
			// if everything is a float we can just copy the float without a constructor.
			FString CurrLine = FString::Printf(TEXT("\tOut.MRT[%d] = MrtFloat%d;\n"), I, I);
			FullStr += CurrLine;
		}
		else
		{
			FString CurrLine = FString::Printf(TEXT("\tOut.MRT[%d] = float4("), I);

			for (int32 Chan = 0; Chan < 4; Chan++)
			{
				if (PackedStatus[I][Chan] == -1)
				{
					CurrLine += "0.0f";
				}
				else
				{
					if (PackedStatus[I][Chan] == 0)
					{
						// raw float
						FString TargetName = FString::Printf(TEXT("MrtFloat%d"), I);
						CurrLine += FString::Printf(TEXT("%s.%s"),
							TargetName.GetCharArray().GetData(),
							Swizzles[Chan].GetCharArray().GetData());
					}
					else
					{
						// packed int, convert to float
						FString TargetName = FString::Printf(TEXT("MrtUint%d"), I);
						int32 ChanBits = GetBufferNumBits(BufferInfo.Targets[I].TargetType, Chan);

						CurrLine += FString::Printf(TEXT("float(%s.%s) / %u.0f"),
							TargetName.GetCharArray().GetData(),
							Swizzles[Chan].GetCharArray().GetData(),
							(1 << ChanBits) - 1);
					}
				}
				if (Chan < 3)
				{
					CurrLine += TEXT(", ");
				}
			}

			CurrLine += TEXT(");\n");
			FullStr += CurrLine;
		}

	}

	FullStr += TEXT("}\n");
	FullStr += TEXT("\n");

	return FullStr;
}

static FString CreateGBufferDecodeFunctionDirect(const FGBufferInfo& BufferInfo)
{
	FString FullStr;


	FullStr += TEXT("FGBufferData  DecodeGBufferDataDirect(");
	bool bFirst = true;
	for (int32 Index = 0; Index < FGBufferInfo::MaxTargets; Index++)
	{
		const EGBufferType Target = BufferInfo.Targets[Index].TargetType;

		if (Target != GBT_Invalid && Index != 0)
		{
			if (bFirst)
			{
				bFirst = false;
			}
			else
			{
				FullStr += TEXT(",\n\t");
			}

			int32 NumChan = GetTargetNumChannels(Target);
			FString TypeName = GetFloatType(NumChan);
			FString CurrLine = FString::Printf(TEXT("%s InMRT%d"),
				TypeName.GetCharArray().GetData(),
				Index);

			FullStr += CurrLine;
		}
	}
	if (!bFirst)
	{
		FullStr += TEXT(",\n\t\t");
	}
	FullStr += TEXT(" \n\tfloat CustomNativeDepth");
	FullStr += TEXT(",\n\tfloat4 AnisotropicData");
	FullStr += TEXT(",\n\tfloat4 SGGXData");
	FullStr += TEXT(",\n\tuint CustomStencil");
	FullStr += TEXT(",\n\tfloat SceneDepth");
	FullStr += TEXT(",\n\tbool bGetNormalizedNormal");
	FullStr += TEXT(",\n\tbool bChecker");
	FullStr += TEXT(",\n\tbool bForceUnlitOrDefaultLit = false)\n");

	FullStr += TEXT("{\n");

	FullStr += TEXT("\tFGBufferData Ret = (FGBufferData)0;\n");
	
	// Default initialization in case no gbuffer data are generated when Substrate is enabled to 
	// prevent shader compiler error (division by zero, variable not-initialized) with passes 
	// not converted to Substrate and still using Gbuffer data
	if (Substrate::IsSubstrateEnabled())
	{
		FullStr += TEXT("\tRet.WorldNormal = float3(0,0,1);\n");
		FullStr += TEXT("\tRet.Depth = 0.f;\n");
		FullStr += TEXT("\tRet.ShadingModelID = 1;\n"); 
	}

	FString Swizzles[4];
	Swizzles[0] = TEXT("x");
	Swizzles[1] = TEXT("y");
	Swizzles[2] = TEXT("z");
	Swizzles[3] = TEXT("w");

	for (int32 SlotIndex = 0; SlotIndex < GBS_Num; SlotIndex++)
	{
		const FGBufferItem& SlotItem = BufferInfo.Slots[SlotIndex];
		if (SlotItem.bIsValid && SlotItem.BufferSlot != GBS_SceneColor)
		{
			int32 DstChanNum = GetCompressionDstNumChannels(SlotItem.Compression);

			bool bIsPackedBits = IsPackedBits(SlotItem.Compression);
			bool bNeedsConversion = GetCompressionNeedsConversion(SlotItem.Compression) != 0;

			FString DstName = GetSlotTextName(SlotItem.BufferSlot);
			if (bNeedsConversion)
			{
				DstName = DstName + TEXT("_Compressed");

				// also, if it is converted, we need to declare the temporary value.
				FString TypeName;
				if (bIsPackedBits)
				{
					TypeName = GetUintType(DstChanNum);
				}
				else
				{
					TypeName = GetFloatType(DstChanNum);
				}

				FullStr += FString::Printf(TEXT("\t%s %s = 0.0f;\n"),
							TypeName.GetCharArray().GetData(),
							DstName.GetCharArray().GetData());
			}
			else
			{
				DstName = TEXT("Ret.") + DstName;
			}

			for (int32 ChanIndex = 0; ChanIndex < DstChanNum; ChanIndex++)
			{
				// first, count how many packing items we have
				int32 NumFound = 0;

				check(FGBufferItem::MaxPacking == 8);
				int32 FoundIndexList[8] = { -1, -1, -1, -1, -1, -1, -1, -1 };

				for (int32 PackIter = 0; PackIter < FGBufferItem::MaxPacking; PackIter++)
				{
					const FGBufferPacking& PackItem = SlotItem.Packing[PackIter];
					if (PackItem.bIsValid && PackItem.SrcChannelIndex == ChanIndex)
					{
						FoundIndexList[NumFound] = PackIter;
						NumFound++;
					}
				}

				if (NumFound == 0)
				{
					// channel is missing
					check(0);
				}
				else if (NumFound == 1)
				{
					check(FoundIndexList[0] >= 0);
					int32 FoundIndex = FoundIndexList[0];

					const FGBufferPacking& PackItem = SlotItem.Packing[FoundIndex];

					if (bIsPackedBits)
					{
						int32 Mask = (1 << PackItem.BitNum)-1;
						
						const EGBufferType TargetType = BufferInfo.Targets[PackItem.TargetIndex].TargetType;
						int32 TargetBitNum = GetBufferNumBits(TargetType,PackItem.DstChannelIndex);
						int32 TargetScale = (1 << TargetBitNum) - 1;

						FString CurrLine = FString::Printf(TEXT("\t%s.%s = (((uint((float(InMRT%d.%s) * %d.0f) + .5f) >> %d) & 0x%02x) << %d);\n"),
							DstName.GetCharArray().GetData(),
							Swizzles[PackItem.SrcChannelIndex].GetCharArray().GetData(),
							PackItem.TargetIndex,
							Swizzles[PackItem.DstChannelIndex].GetCharArray().GetData(),
							TargetScale,
							PackItem.DstBitStart,
							Mask,
							PackItem.SrcBitStart);
						
						FullStr += CurrLine;
					}
					else
					{
						// simple, just copy from float to float
						FString CurrLine = FString::Printf(TEXT("\t%s.%s = InMRT%d.%s;\n"),
							DstName.GetCharArray().GetData(),
							Swizzles[PackItem.SrcChannelIndex].GetCharArray().GetData(),
							PackItem.TargetIndex,
							Swizzles[PackItem.DstChannelIndex].GetCharArray().GetData());
						FullStr += CurrLine;
					}
				}
				else
				{
					// reconstruct bits from multiple places, not implemented yet
					check(FoundIndexList[0] >= 0);
					check(bIsPackedBits);

					for (int32 FoundIter = 0; FoundIter < NumFound; FoundIter++)
					{
						int32 FoundIndex = FoundIndexList[FoundIter];

						const FGBufferPacking& PackItem = SlotItem.Packing[FoundIndex];

						int32 Mask = (1 << PackItem.BitNum) - 1;

						const EGBufferType TargetType = BufferInfo.Targets[PackItem.TargetIndex].TargetType;
						int32 TargetBitNum = GetBufferNumBits(TargetType, PackItem.DstChannelIndex);
						int32 TargetScale = (1 << TargetBitNum) - 1;

						FString AssignmentOperator = (FoundIter == 0) ? TEXT("=") : TEXT("|=");

						FString CurrLine = FString::Printf(TEXT("\t%s.%s %s (((uint((float(InMRT%d.%s) * %d.0f) + 0.5f) >> %d) & 0x%02x) << %d);\n"),
							DstName.GetCharArray().GetData(),
							Swizzles[PackItem.SrcChannelIndex].GetCharArray().GetData(),
							AssignmentOperator.GetCharArray().GetData(),
							PackItem.TargetIndex,
							Swizzles[PackItem.DstChannelIndex].GetCharArray().GetData(),
							TargetScale,
							PackItem.DstBitStart,
							Mask,
							PackItem.SrcBitStart);

						FullStr += CurrLine;
					}
				}
			}
		}
	}

	FullStr += TEXT("\t\n");

	// check for anything that needs an extra decode step
	for (int32 SlotIndex = 0; SlotIndex < GBS_Num; SlotIndex++)
	{
		const FGBufferItem& SlotItem = BufferInfo.Slots[SlotIndex];
		if (SlotItem.bIsValid && SlotItem.BufferSlot != GBS_SceneColor)
		{
			int32 DstChanNum = GetCompressionDstNumChannels(SlotItem.Compression);

			bool bIsPackedBits = IsPackedBits(SlotItem.Compression);
			bool bNeedsConversion = GetCompressionNeedsConversion(SlotItem.Compression) != 0;

			FString DstName = GetSlotTextName(SlotItem.BufferSlot);
			if (bNeedsConversion)
			{
				FString CompressName = DstName + TEXT("_Compressed");
				FString FuncName = GetCompressionDecodeFuncName(SlotItem.Compression);

				// also, if it is converted, we need to declare the temporary value.
				FString TypeName = GetFloatType(DstChanNum);
				FullStr += FString::Printf(TEXT("\tRet.%s = %s(%s);\n"),
					DstName.GetCharArray().GetData(),
					FuncName.GetCharArray().GetData(),
					CompressName.GetCharArray().GetData());
			}
		}
	}

	FullStr += TEXT("\tif(bForceUnlitOrDefaultLit && Ret.ShadingModelID != 0) Ret.ShadingModelID = 1;\n");

	FullStr += TEXT("\tRet.WorldTangent = AnisotropicData.xyz;\n");
	FullStr += TEXT("\tRet.Anisotropy = AnisotropicData.w;\n");

	FullStr += TEXT("\tRet.NormalDistribution = DecodeNormal(SGGXData.xyz);\n");

	FullStr += TEXT("\n");
	FullStr += TEXT("\tGBufferPostDecode(Ret,bChecker,bGetNormalizedNormal);\n");
	FullStr += TEXT("\n");

	//// Hacks for now, fixme later
	FullStr += TEXT("\tRet.CustomDepth = ConvertFromDeviceZ(CustomNativeDepth);\n");
	FullStr += TEXT("\tRet.CustomStencil = CustomStencil;\n");
	FullStr += TEXT("\tRet.Depth = SceneDepth;\n");
	FullStr += TEXT("\t\n");

	FullStr += TEXT("\n");
	FullStr += TEXT("\treturn Ret;\n");

	FullStr += TEXT("}\n");
	FullStr += TEXT("\n");

	return FullStr;
}


enum EGBufferDecodeType
{
	CoordUV			 , // GetGBufferData
	CoordUInt		 , // GetGBufferDataUint
	SceneTextures,    // GetGBufferDataFromSceneTextures
	SceneTexturesLoad, // GetGBufferDataFromSceneTexturesLoad
	Num
};

struct FGBufferDecoderSyntax
{
	const TCHAR * Suffix;
	const TCHAR * CoordType;
	const TCHAR * CoordName;
	const TCHAR * CheckerFunc;
};

static FGBufferDecoderSyntax GDecoderSyntax[EGBufferDecodeType::Num] =
{
	{ TEXT("UV"), TEXT("float2"), TEXT("UV"), TEXT("CheckerFromSceneColorUV") },
	{ TEXT("Uint"), TEXT("uint2"), TEXT("PixelPos"), TEXT("CheckerFromPixelPos") },
	{ TEXT("SceneTextures"), TEXT("float2"), TEXT("UV"), TEXT("CheckerFromSceneColorUV") },
	{ TEXT("SceneTexturesLoad"), TEXT("uint2"), TEXT("PixelCoord"), TEXT("CheckerFromPixelPos") },
};

static FString CreateGBufferDecodeFunctionVariation(const FGBufferInfo& BufferInfo, EGBufferDecodeType DecodeType, ERHIFeatureLevel::Type FEATURE_LEVEL)
{
	FString FullStr;

	FString Suffix = GDecoderSyntax[DecodeType].Suffix;
	FString CoordType = GDecoderSyntax[DecodeType].CoordType;
	FString CoordName = GDecoderSyntax[DecodeType].CoordName;
	FString CheckerFunc = GDecoderSyntax[DecodeType].CheckerFunc;


	FullStr += TEXT("// @param PixelPos relative to left top of the rendertarget (not viewport)\n");

	FullStr += FString::Printf(TEXT("FGBufferData DecodeGBufferData%s(%s %s, bool bGetNormalizedNormal = true, bool bForceUnlitOrDefaultLit = false)\n"),
		Suffix.GetCharArray().GetData(),
		CoordType.GetCharArray().GetData(),
		CoordName.GetCharArray().GetData());

	FullStr += TEXT("{\n");

	if (DecodeType == CoordUV)
	{
		FullStr += FString::Printf(TEXT("\tfloat CustomNativeDepth = Texture2DSampleLevel(SceneTexturesStruct.CustomDepthTexture, SceneTexturesStruct_CustomDepthTextureSampler, %s, 0).r;\n"), CoordName.GetCharArray().GetData());

		// BufferToSceneTextureScale is necessary when translucent materials are rendered in a render target 
		// that has a different resolution than the scene color textures, e.g. r.SeparateTranslucencyScreenPercentage < 100.
		FullStr += FString::Printf(TEXT("\tint2 IntUV = (int2)trunc(%s * View.BufferSizeAndInvSize.xy * View.BufferToSceneTextureScale.xy);\n"), CoordName.GetCharArray().GetData());
		FullStr += TEXT("\tuint CustomStencil = SceneTexturesStruct.CustomStencilTexture.Load(int3(IntUV, 0)) STENCIL_COMPONENT_SWIZZLE;\n");

		FullStr += FString::Printf(TEXT("\tfloat SceneDepth = CalcSceneDepth(%s);\n"), CoordName.GetCharArray().GetData());
		FullStr += TEXT("\tfloat4 AnisotropicData = Texture2DSampleLevel(SceneTexturesStruct.GBufferFTexture, SceneTexturesStruct_GBufferFTextureSampler, UV, 0).xyzw;\n");
		FullStr += TEXT("\tfloat4 SGGXData = Texture2DSampleLevel(SceneTexturesStruct.GBufferSGGXTexture, SceneTexturesStruct_SGGXTextureSampler, UV, 0).xyzw;\n");
	}
	else if (DecodeType == CoordUInt)
	{
		FullStr += FString::Printf(TEXT("\tfloat CustomNativeDepth = SceneTexturesStruct.CustomDepthTexture.Load(int3(%s, 0)).r;\n"), CoordName.GetCharArray().GetData());
		FullStr += FString::Printf(TEXT("\tuint CustomStencil = SceneTexturesStruct.CustomStencilTexture.Load(int3(%s, 0)) STENCIL_COMPONENT_SWIZZLE;\n"), CoordName.GetCharArray().GetData());
		FullStr += FString::Printf(TEXT("\tfloat SceneDepth = CalcSceneDepth(%s);\n"), CoordName.GetCharArray().GetData());
		FullStr += TEXT("\tfloat4 AnisotropicData = SceneTexturesStruct.GBufferFTexture.Load(int3(PixelPos, 0)).xyzw;\n");
		FullStr += TEXT("\tfloat4 SGGXData = SceneTexturesStruct.GBufferSGGXTexture.Load(int3(PixelPos, 0)).xyzw;\n");
	}
	else if (DecodeType == SceneTextures)
	{
		FullStr += TEXT("\tuint CustomStencil = 0;\n");
		FullStr += TEXT("\tfloat CustomNativeDepth = 0;\n");
		FullStr += FString::Printf(TEXT("\tfloat DeviceZ = SampleDeviceZFromSceneTexturesTempCopy(%s);\n"), CoordName.GetCharArray().GetData());
		FullStr += TEXT("\tfloat SceneDepth = ConvertFromDeviceZ(DeviceZ);\n");
		FullStr += TEXT("\tfloat4 AnisotropicData = GBufferFTexture.SampleLevel(GBufferFTextureSampler, UV, 0).xyzw;\n");
		FullStr += TEXT("\tfloat4 SGGXData = GBufferSGGXTexture.SampleLevel(GBufferSGGXTextureSampler, UV, 0).xyzw;\n");
	}
	else if (DecodeType == SceneTexturesLoad)
	{
		FullStr += TEXT("\tuint CustomStencil = 0;\n");
		FullStr += TEXT("\tfloat CustomNativeDepth = 0;\n");
		FullStr += FString::Printf(TEXT("\tfloat DeviceZ = SceneDepthTexture.Load(int3(%s, 0)).r;\n"), CoordName.GetCharArray().GetData());
		FullStr += TEXT("\tfloat SceneDepth = ConvertFromDeviceZ(DeviceZ);\n");
		FullStr += TEXT("\tfloat4 AnisotropicData = GBufferFTexture.Load(int3(PixelCoord, 0)).xyzw;\n");
		FullStr += TEXT("\tfloat4 SGGXData = GBufferSGGXTexture.Load(int3(PixelCoord, 0)).xyzw;\n");
	}
	else
	{
		check(0);
	}

	FullStr += TEXT("\n");

	FString FullSwizzle[4];
	FullSwizzle[0] = TEXT("x");
	FullSwizzle[1] = TEXT("xy");
	FullSwizzle[2] = TEXT("xyz");
	FullSwizzle[3] = TEXT("xyzw");

	const bool bAllowCustomDepthStencil = ((DecodeType == EGBufferDecodeType::CoordUInt && FEATURE_LEVEL >= ERHIFeatureLevel::SM5) || DecodeType == EGBufferDecodeType::CoordUV);

	for (int32 Index = 0; Index < FGBufferInfo::MaxTargets; Index++)
	{
		const EGBufferType Target = BufferInfo.Targets[Index].TargetType;

		if (Target != GBT_Invalid && Index != 0)
		{
			int32 NumChan = GetTargetNumChannels(Target);
			FString TypeName = GetFloatType(NumChan);
			FString TargetName = BufferInfo.Targets[Index].TargetName;

			// special exception for WRITES_VELOCITY_TO_GBUFFER
			if (TargetName == TEXT("Velocity"))
			{
				if (DecodeType == CoordUV || DecodeType == CoordUInt)
				{
					TargetName = TEXT("GBufferVelocity");
				}
				else if (DecodeType == SceneTextures || DecodeType == SceneTexturesLoad)
				{
					TargetName = TEXT("GBufferVelocity");
				}
			}

			FString CurrLine;
			if (DecodeType == CoordUV)
			{
				CurrLine = FString::Printf(TEXT("\t%s InMRT%d = Texture2DSampleLevel(SceneTexturesStruct.%sTexture, SceneTexturesStruct_%sTextureSampler, %s, 0).%s;\n"),
					TypeName.GetCharArray().GetData(),
					Index,
					TargetName.GetCharArray().GetData(),
					TargetName.GetCharArray().GetData(),
					CoordName.GetCharArray().GetData(),
					FullSwizzle[NumChan-1].GetCharArray().GetData());
			}
			else if (DecodeType == CoordUInt)
			{
				CurrLine = FString::Printf(TEXT("\t%s InMRT%d = SceneTexturesStruct.%sTexture.Load(int3(%s, 0)).%s;\n"),
					TypeName.GetCharArray().GetData(),
					Index,
					TargetName.GetCharArray().GetData(),
					CoordName.GetCharArray().GetData(),
					FullSwizzle[NumChan-1].GetCharArray().GetData());
			}
			else if (DecodeType == SceneTextures)
			{
				CurrLine = FString::Printf(TEXT("\t%s InMRT%d = %sTexture.SampleLevel(%sTextureSampler, %s, 0).%s;\n"),
					TypeName.GetCharArray().GetData(),
					Index,
					TargetName.GetCharArray().GetData(),
					TargetName.GetCharArray().GetData(),
					CoordName.GetCharArray().GetData(),
					FullSwizzle[NumChan-1].GetCharArray().GetData());
			}
			else if (DecodeType == SceneTexturesLoad) //-V547
			{
				CurrLine = FString::Printf(TEXT("\t%s InMRT%d = %sTexture.Load(int3(%s, 0)).%s;\n"),
					TypeName.GetCharArray().GetData(),
					Index,
					TargetName.GetCharArray().GetData(),
					CoordName.GetCharArray().GetData(),
					FullSwizzle[NumChan-1].GetCharArray().GetData());
			}
			else
			{
				check(0);
			}

			FullStr += CurrLine;
		}
	}

	FullStr += TEXT("\n");

	FullStr += TEXT("\tFGBufferData Ret = DecodeGBufferDataDirect(");
	bool bFirst = true;
	for (int32 Index = 0; Index < FGBufferInfo::MaxTargets; Index++)
	{
		const EGBufferType Target = BufferInfo.Targets[Index].TargetType;

		if (Target != GBT_Invalid && Index != 0)
		{
			if (bFirst)
			{
				bFirst = false;
			}
			else
			{
				FullStr += TEXT(",\n\t\t");
			}

			int32 NumChan = GetTargetNumChannels(Target);
			FString CurrLine = FString::Printf(TEXT("InMRT%d"),
				Index);

			FullStr += CurrLine;
		}
	}

	if (!bFirst)
	{
		FullStr += TEXT(",\n\t\t");
	}
	FullStr += TEXT(" \n\t\tCustomNativeDepth");
	FullStr += TEXT(",\n\t\tAnisotropicData");
	FullStr += TEXT(",\n\t\tSGGXData");
	FullStr += TEXT(",\n\t\tCustomStencil");
	FullStr += TEXT(",\n\t\tSceneDepth");
	FullStr += TEXT(",\n\t\tbGetNormalizedNormal");
	FullStr += FString::Printf(TEXT(",\n\t\t%s(%s)"),
					CheckerFunc.GetCharArray().GetData(),
					CoordName.GetCharArray().GetData());
	FullStr += TEXT(",\n\t\tbForceUnlitOrDefaultLit);\n");

	FullStr += TEXT("\n");
	FullStr += TEXT("\treturn Ret;\n");

	FullStr += TEXT("}\n");
	FullStr += TEXT("\n");

	return FullStr;
}

static FCriticalSection GCriticalSection;

static FString GetAutoGenDirectory(EShaderPlatform TargetPlatform)
{
	FString PlatformName = FDataDrivenShaderPlatformInfo::GetName(TargetPlatform).ToString();
	FString AutogenHeaderDirectory = FPaths::ProjectIntermediateDir() / TEXT("ShaderAutogen") / PlatformName;
	return AutogenHeaderDirectory;
}



static void SetSharedGBufferSlots(bool Slots[])
{
	Slots[GBS_SceneColor] = true;
	Slots[GBS_WorldNormal] = true;
	Slots[GBS_PerObjectGBufferData] = true;
	Slots[GBS_Metallic] = true;
	Slots[GBS_Specular] = true;
	Slots[GBS_Roughness] = true;
	Slots[GBS_ShadingModelId] = true;
	Slots[GBS_SelectiveOutputMask] = true;
	Slots[GBS_BaseColor] = true;
	Slots[GBS_GenericAO] = true;
	Slots[GBS_AO] = false;// true; // figure this out later
}

// If we are storing writing to the GBuffer, what slots does this shading model write? Ignores the other forward-shading effects
// like the dual blending of ThinTranslucent or fogging.
// If we are going one GBuffer for all shader model formats, then we treat all custom data as a single RGBA8 value (bMergeCustom=true). But
// if each shading model has a unique GBuffer format, we set it to false to pack each format as tightly as possible.
static void SetSlotsForShadingModelType(bool Slots[], EMaterialShadingModel ShadingModel, bool bMergeCustom)
{
	switch (ShadingModel)
	{
	case MSM_Unlit:
		Slots[GBS_SceneColor] = true;
		break;
	case MSM_DefaultLit:
		SetSharedGBufferSlots(Slots);
		break;
	case MSM_Subsurface:
		SetSharedGBufferSlots(Slots);
		if (bMergeCustom)
		{
			Slots[GBS_CustomData] = true;
		}
		else
		{
			Slots[GBS_SubsurfaceColor] = true;
			Slots[GBS_Opacity] = true;
		}
		break;
	case MSM_PreintegratedSkin:
		SetSharedGBufferSlots(Slots);
		if (bMergeCustom)
		{
			Slots[GBS_CustomData] = true;
		}
		else
		{
			Slots[GBS_SubsurfaceColor] = true;
			Slots[GBS_Opacity] = true;
		}
		break;
	case MSM_ClearCoat:
		SetSharedGBufferSlots(Slots);
		if (bMergeCustom)
		{
			Slots[GBS_CustomData] = true;
		}
		else
		{
			Slots[GBS_ClearCoat] = true;
			Slots[GBS_ClearCoatRoughness] = true;
		}
		break;
	case MSM_SubsurfaceProfile:
		SetSharedGBufferSlots(Slots);
		if (bMergeCustom)
		{
			Slots[GBS_CustomData] = true;
		}
		else
		{
			Slots[GBS_SubsurfaceProfile] = true;
			Slots[GBS_Opacity] = true;
		}
		break;
	case MSM_TwoSidedFoliage:
		SetSharedGBufferSlots(Slots);
		if (bMergeCustom)
		{
			Slots[GBS_CustomData] = true;
		}
		else
		{
			Slots[GBS_SubsurfaceColor] = true;
			Slots[GBS_Opacity] = true;
		}
		break;
	case MSM_Hair:
		SetSharedGBufferSlots(Slots);
		if (bMergeCustom)
		{
			Slots[GBS_CustomData] = true;
		}
		else
		{
			Slots[GBS_HairSecondaryWorldNormal] = true;
			Slots[GBS_HairBacklit] = true;
		}
		break;
	case MSM_Cloth:
		SetSharedGBufferSlots(Slots);
		if (bMergeCustom)
		{
			Slots[GBS_CustomData] = true;
		}
		else
		{
			Slots[GBS_SubsurfaceColor] = true;
			Slots[GBS_Cloth] = true;
		}
		break;
	case MSM_Eye:
		SetSharedGBufferSlots(Slots);
		if (bMergeCustom)
		{
			Slots[GBS_CustomData] = true;
		}
		else
		{
			Slots[GBS_SubsurfaceProfile] = true;
			Slots[GBS_IrisNormal] = true;
			Slots[GBS_Opacity] = true;
		}
		break;
	case MSM_SingleLayerWater:
		SetSharedGBufferSlots(Slots);
		Slots[GBS_SeparatedMainDirLight] = true;
		break;
	case MSM_ThinTranslucent:
		// thin translucent doesn't write to the GBuffer
		break;
	}
}

enum class EGBufferSlotUsage : uint8
{
	Unused,
	Used,
	Written
};

static bool operator> (EGBufferSlotUsage A, EGBufferSlotUsage B) { return uint8(A) >  uint8(B); }
static bool operator>=(EGBufferSlotUsage A, EGBufferSlotUsage B) { return uint8(A) >= uint8(B); }
static bool operator< (EGBufferSlotUsage A, EGBufferSlotUsage B) { return uint8(A) <  uint8(B); }
static bool operator<=(EGBufferSlotUsage A, EGBufferSlotUsage B) { return uint8(A) <= uint8(B); }

static EGBufferSlotUsage GetGBufferSlotUsage(bool bWritten, bool bForceUsed = false)
{
	return bWritten ? EGBufferSlotUsage::Written : (bForceUsed ? EGBufferSlotUsage::Used : EGBufferSlotUsage::Unused);
}

static void SetStandardGBufferSlots(
	EGBufferSlotUsage Slots[GBS_Num],
	bool bWriteEmissive,
	bool bHasTangent,
	bool bHasVelocity,
	bool bWritesVelocity,
	bool bHasStaticLighting,
	bool bIsSubstrateMaterial,
	bool bIsSubstrateMaterialNewGBuffer)
{
	auto GetUsageIfNotSubstrate = [bIsSubstrateMaterial, bIsSubstrateMaterialNewGBuffer](bool bWritten, bool bForceUsed = false)
	{
		return GetGBufferSlotUsage(bWritten && (!bIsSubstrateMaterial || !bIsSubstrateMaterialNewGBuffer), bForceUsed && (!bIsSubstrateMaterial || !bIsSubstrateMaterialNewGBuffer));
	};

	Slots[GBS_SceneColor]				= GetGBufferSlotUsage(bWriteEmissive);
	Slots[GBS_Velocity]					= GetGBufferSlotUsage(bWritesVelocity, bHasVelocity);
	Slots[GBS_PrecomputedShadowFactor]	= GetGBufferSlotUsage(bHasStaticLighting);

	Slots[GBS_WorldNormal] 				= GetUsageIfNotSubstrate(true);
	Slots[GBS_PerObjectGBufferData]		= GetUsageIfNotSubstrate(true);
	Slots[GBS_Metallic] 				= GetUsageIfNotSubstrate(true);
	Slots[GBS_Specular] 				= GetUsageIfNotSubstrate(true);
	Slots[GBS_Roughness] 				= GetUsageIfNotSubstrate(true);
	Slots[GBS_ShadingModelId] 			= GetUsageIfNotSubstrate(true);
	Slots[GBS_SelectiveOutputMask] 		= GetUsageIfNotSubstrate(true);
	Slots[GBS_BaseColor] 				= GetUsageIfNotSubstrate(true);
	Slots[GBS_GenericAO] 				= GetUsageIfNotSubstrate(true);

	Slots[GBS_WorldTangent] 			= GetUsageIfNotSubstrate(bHasTangent);
	Slots[GBS_Anisotropy]				= GetUsageIfNotSubstrate(bHasTangent);

	Slots[GBS_AO] 						= EGBufferSlotUsage::Unused;
}

static void DetermineUsedMaterialSlots(
	EGBufferSlotUsage Slots[GBS_Num],
	const FShaderMaterialDerivedDefines& Dst,
	const FShaderMaterialPropertyDefines& Mat,
	const FShaderLightmapPropertyDefines& Lightmap,
	const FShaderGlobalDefines& SrcGlobal,
	const FShaderCompilerDefines& Compiler,
	ERHIFeatureLevel::Type FEATURE_LEVEL)
{
	bool bWriteEmissive = Dst.NEEDS_BASEPASS_VERTEX_FOGGING || Mat.USES_EMISSIVE_COLOR || SrcGlobal.ALLOW_STATIC_LIGHTING || Mat.MATERIAL_SHADINGMODEL_SINGLELAYERWATER;
	bool bHasTangent = SrcGlobal.GBUFFER_HAS_TANGENT;
	bool bHasVelocity = SrcGlobal.GBUFFER_HAS_VELOCITY;
	bool bWritesVelocity = Dst.WRITES_VELOCITY_TO_GBUFFER;
	bool bHasStaticLighting = Dst.GBUFFER_HAS_PRECSHADOWFACTOR || Dst.WRITES_PRECSHADOWFACTOR_TO_GBUFFER;
	bool bIsSubstrateMaterial = Mat.SUBSTRATE_ENABLED; // Similarly to FetchFullGBufferInfo, we do not check for MATERIAL_IS_SUBSTRATE as this is decided per project.
	bool bIsSubstrateNewGBuffer = Mat.SUBSTRATE_GBUFFER_FORMAT==1; // Similarly to FetchFullGBufferInfo, we do not check for MATERIAL_IS_SUBSTRATE as this is decided per project.

	// Substrate doesn't use gbuffer, and thus doesn't need CustomData
	const bool bUseCustomData = !bIsSubstrateMaterial || !bIsSubstrateNewGBuffer;

	// we have to use if statements, not switch or if/else statements because we can have multiple shader model ids.
	if (Mat.MATERIAL_SHADINGMODEL_UNLIT)
	{
		// With Substrate, we still need to write to the GBufferE when static lighting is used to maintain the correct offset for SubstrateFirstMRT (and match the bound render targets).
		// Without Substrate, the buffer is always bound and written to. It is just not used and a default value was used.
		const bool bHasStaticLightingWritten = bIsSubstrateMaterial ? bHasStaticLighting : false;
		SetStandardGBufferSlots(Slots, true, false, bHasVelocity, bWritesVelocity, bHasStaticLightingWritten, bIsSubstrateMaterial, bIsSubstrateNewGBuffer);
	}

	if (Mat.MATERIAL_SHADINGMODEL_DEFAULT_LIT)
	{
		SetStandardGBufferSlots(Slots, bWriteEmissive, bHasTangent, bHasVelocity, bWritesVelocity, bHasStaticLighting, bIsSubstrateMaterial, bIsSubstrateNewGBuffer);
	}

	if (Mat.MATERIAL_SHADINGMODEL_SUBSURFACE)
	{
		SetStandardGBufferSlots(Slots, bWriteEmissive, bHasTangent, bHasVelocity, bWritesVelocity, bHasStaticLighting, bIsSubstrateMaterial, bIsSubstrateNewGBuffer);
		Slots[GBS_CustomData] = GetGBufferSlotUsage(bUseCustomData);
	}

	if (Mat.MATERIAL_SHADINGMODEL_PREINTEGRATED_SKIN)
	{
		SetStandardGBufferSlots(Slots, bWriteEmissive, bHasTangent, bHasVelocity, bWritesVelocity, bHasStaticLighting, bIsSubstrateMaterial, bIsSubstrateNewGBuffer);
		Slots[GBS_CustomData] = GetGBufferSlotUsage(bUseCustomData);
	}

	if (Mat.MATERIAL_SHADINGMODEL_SUBSURFACE_PROFILE)
	{
		SetStandardGBufferSlots(Slots, bWriteEmissive, bHasTangent, bHasVelocity, bWritesVelocity, bHasStaticLighting, bIsSubstrateMaterial, bIsSubstrateNewGBuffer);
		Slots[GBS_CustomData] = GetGBufferSlotUsage(bUseCustomData);
	}

	if (Mat.MATERIAL_SHADINGMODEL_CLEAR_COAT)
	{
		SetStandardGBufferSlots(Slots, bWriteEmissive, bHasTangent, bHasVelocity, bWritesVelocity, bHasStaticLighting, bIsSubstrateMaterial, bIsSubstrateNewGBuffer);
		Slots[GBS_CustomData] = GetGBufferSlotUsage(bUseCustomData);
	}

	if (Mat.MATERIAL_SHADINGMODEL_TWOSIDED_FOLIAGE)
	{
		SetStandardGBufferSlots(Slots, bWriteEmissive, bHasTangent, bHasVelocity, bWritesVelocity, bHasStaticLighting, bIsSubstrateMaterial, bIsSubstrateNewGBuffer);
		Slots[GBS_CustomData] = GetGBufferSlotUsage(bUseCustomData);
	}

	if (Mat.MATERIAL_SHADINGMODEL_HAIR)
	{
		SetStandardGBufferSlots(Slots, bWriteEmissive, bHasTangent, bHasVelocity, bWritesVelocity, bHasStaticLighting, bIsSubstrateMaterial, bIsSubstrateNewGBuffer);
		Slots[GBS_CustomData] = GetGBufferSlotUsage(bUseCustomData);
	}

	if (Mat.MATERIAL_SHADINGMODEL_CLOTH)
	{
		SetStandardGBufferSlots(Slots, bWriteEmissive, bHasTangent, bHasVelocity, bWritesVelocity, bHasStaticLighting, bIsSubstrateMaterial, bIsSubstrateNewGBuffer);
		Slots[GBS_CustomData] = GetGBufferSlotUsage(bUseCustomData);
	}

	if (Mat.MATERIAL_SHADINGMODEL_EYE)
	{
		SetStandardGBufferSlots(Slots, bWriteEmissive, bHasTangent, bHasVelocity, bWritesVelocity, bHasStaticLighting, bIsSubstrateMaterial, bIsSubstrateNewGBuffer);
		Slots[GBS_CustomData] = GetGBufferSlotUsage(bUseCustomData);
	}

	if (Mat.MATERIAL_SHADINGMODEL_TOON)
	{
		SetStandardGBufferSlots(Slots, bWriteEmissive, bHasTangent, bHasVelocity, bWritesVelocity, bHasStaticLighting, bIsSubstrateMaterial, bIsSubstrateNewGBuffer);
		Slots[GBS_CustomData] = GetGBufferSlotUsage(bUseCustomData);
	}

	if (Mat.MATERIAL_SHADINGMODEL_SINGLELAYERWATER)
	{
		// single layer water uses standard slots
		SetStandardGBufferSlots(Slots, bWriteEmissive, bHasTangent, bHasVelocity, bWritesVelocity, bHasStaticLighting, bIsSubstrateMaterial, bIsSubstrateNewGBuffer);
		if (Mat.SINGLE_LAYER_WATER_SEPARATED_MAIN_LIGHT)
		{
			Slots[GBS_SeparatedMainDirLight] = EGBufferSlotUsage::Written;
		}
	}

	// doesn't write to GBuffer
	if (Mat.MATERIAL_SHADINGMODEL_THIN_TRANSLUCENT)
	{
	}
}

void FShaderCompileUtilities::ApplyDerivedDefines(FShaderCompilerEnvironment& OutEnvironment, FShaderCompilerEnvironment * SharedEnvironment, const EShaderPlatform Platform)
{
	ERHIFeatureLevel::Type FeatureLevel = GetMaxSupportedFeatureLevel(Platform);

	FShaderMaterialPropertyDefines MaterialDefines = {};
	FShaderLightmapPropertyDefines LightmapDefines = {};
	FShaderGlobalDefines GlobalDefines = {};
	FShaderCompilerDefines CompilerDefines = {};
	FShaderMaterialDerivedDefines DerivedDefines = {};

	ApplyFetchEnvironment(GlobalDefines, OutEnvironment, Platform);
	ApplyFetchEnvironment(MaterialDefines, OutEnvironment);
	ApplyFetchEnvironment(LightmapDefines, OutEnvironment);
	ApplyFetchEnvironment(CompilerDefines, OutEnvironment);

	if (SharedEnvironment != nullptr)
	{
		ApplyFetchEnvironment(GlobalDefines, *SharedEnvironment, Platform);
		ApplyFetchEnvironment(MaterialDefines, *SharedEnvironment);
		ApplyFetchEnvironment(LightmapDefines, *SharedEnvironment);
		ApplyFetchEnvironment(CompilerDefines, *SharedEnvironment);

		OutEnvironment.FullPrecisionInPS |= SharedEnvironment->FullPrecisionInPS;
	}

	// for reference, here are the existing defines
	DerivedDefines = CalculateDerivedMaterialParameters(MaterialDefines, LightmapDefines, GlobalDefines, CompilerDefines, FeatureLevel);

	// We will need to match these. For existing code, values such as PIXELSHADEROUTPUT_MRT5 are based on a collection of #ifdefs. First, we 
	// need to define those here instead of doing that logic inside the shader code macros. Then as a second step, we need to enable/define
	// the targets based on the FGBufferInfo struct.

	// for now however, we just only set the global enable flag while working on this. Keeping this as a static so it can be changed in debug mode for testing,
	// but not making it a cvar because the old path will be removed once it is confirmed to work.
	static bool bUseRefactor = true;
	OutEnvironment.SetDefine(TEXT("GBUFFER_REFACTOR"), bUseRefactor ? TEXT("1") : TEXT("0"));

	EGBufferLayout Layout = (EGBufferLayout)MaterialDefines.GBUFFER_LAYOUT;
	FGBufferParams Params = FShaderCompileUtilities::FetchGBufferParamsRuntime(Platform, Layout);
	FGBufferInfo BufferInfo = FetchFullGBufferInfo(Params);	

	EGBufferSlotUsage TargetUsage[FGBufferInfo::MaxTargets] = {};
	if (MaterialDefines.IS_BASE_PASS)
	{
		// if we are using a gbuffer, and this is the base pass that writes a gbuffer, search the gbuffer for each slot
		if (DerivedDefines.USES_GBUFFER)
		{
			TargetUsage[0] = EGBufferSlotUsage::Written;
			EGBufferSlotUsage Slots[GBS_Num] = {};

			DetermineUsedMaterialSlots(Slots, DerivedDefines, MaterialDefines, LightmapDefines, GlobalDefines, CompilerDefines, FeatureLevel);

			// for each slot, which MRT index does it use? -1 if unused.
			int32 SlotTargets[GBS_Num];
			for (int32 Index = 0; Index < GBS_Num; Index++)
			{
				SlotTargets[Index] = -1;
			}

			for (int32 Index = 0; Index < GBS_Num; Index++)
			{
				// if we are using this slot
				if (Slots[Index] != EGBufferSlotUsage::Unused)
				{
					// if we are using this slot, it must have a valid spot in our gbuffer
					const FGBufferItem& Item = BufferInfo.Slots[Index];
					check(Item.bIsValid);

					for (int32 PackIter = 0; PackIter < FGBufferItem::MaxPacking; PackIter++)
					{
						const FGBufferPacking& Packing = Item.Packing[PackIter];
						if (Packing.bIsValid)
						{
							check(Packing.TargetIndex >= 0);
							TargetUsage[Packing.TargetIndex] = FMath::Max(TargetUsage[Packing.TargetIndex], Slots[Index]);
						}
					}
				}
			}
		}
		else
		{
			TargetUsage[0] = EGBufferSlotUsage::Written;
			// we also need MRT for thin translucency due to dual blending if we are not on the fallback path
			const bool bDualSourceBlending = MaterialDefines.DUAL_SOURCE_COLOR_BLENDING_ENABLED && DerivedDefines.MATERIAL_WORKS_WITH_DUAL_SOURCE_COLOR_BLENDING;
			const bool bHasVelocity = GlobalDefines.GBUFFER_HAS_VELOCITY;
			const bool bWritesVelocity = DerivedDefines.WRITES_VELOCITY_TO_GBUFFER;
			TargetUsage[1] = GetGBufferSlotUsage(bWritesVelocity || bDualSourceBlending, bHasVelocity);
		}
	}
	else if (MaterialDefines.IS_VIRTUAL_TEXTURE_MATERIAL)
	{
		// these will change, of course
		if (MaterialDefines.OUT_BASECOLOR)
		{
			TargetUsage[0] = EGBufferSlotUsage::Written;
		}
		else if (MaterialDefines.OUT_BASECOLOR_NORMAL_ROUGHNESS)
		{
			TargetUsage[0] = EGBufferSlotUsage::Written;
			TargetUsage[1] = EGBufferSlotUsage::Written;
		}
		else if (MaterialDefines.OUT_BASECOLOR_NORMAL_SPECULAR)
		{
			TargetUsage[0] = EGBufferSlotUsage::Written;
			TargetUsage[1] = EGBufferSlotUsage::Written;
			TargetUsage[2] = EGBufferSlotUsage::Written;
		}
		else if (MaterialDefines.OUT_MASK4)
		{
			TargetUsage[0] = EGBufferSlotUsage::Written;
			TargetUsage[1] = EGBufferSlotUsage::Written;
		}
		else if (MaterialDefines.OUT_WORLDHEIGHT)
		{
			TargetUsage[0] = EGBufferSlotUsage::Written;
		}
		else if (MaterialDefines.OUT_DISPLACEMENT)
		{
			TargetUsage[0] = EGBufferSlotUsage::Written;
		}
	}
	else if (MaterialDefines.IS_DECAL)
	{
		// these will have to change too
		TargetUsage[0] = GetGBufferSlotUsage(MaterialDefines.DECAL_RENDERTARGET_COUNT > 0);
		TargetUsage[1] = GetGBufferSlotUsage(MaterialDefines.DECAL_RENDERTARGET_COUNT > 1);
		TargetUsage[2] = GetGBufferSlotUsage(MaterialDefines.DECAL_RENDERTARGET_COUNT > 2);
		TargetUsage[3] = GetGBufferSlotUsage(MaterialDefines.DECAL_RENDERTARGET_COUNT > 3);
		TargetUsage[4] = GetGBufferSlotUsage(MaterialDefines.DECAL_RENDERTARGET_COUNT > 4);
	}
	else
	{
		// something else, so no op
	}

	// Decide which pixel shader outputs are enabled based on which targets are written and what the first substrate
	// target is based on which target slots are in use
	int32 SubstrateFirstMRT = 0;
	for (int32 Iter = 0; Iter < FGBufferInfo::MaxTargets; Iter++)
	{
		if (TargetUsage[Iter] >= EGBufferSlotUsage::Written)
		{
			FString TargetName = FString::Printf(TEXT("PIXELSHADEROUTPUT_MRT%d"), Iter);
			OutEnvironment.SetDefine(TargetName.GetCharArray().GetData(), TEXT("1"));
		}

		if (TargetUsage[Iter] >= EGBufferSlotUsage::Used)
		{
			SubstrateFirstMRT = Iter + 1;
		}
	}

	if (MaterialDefines.MATERIAL_IS_SUBSTRATE && MaterialDefines.SUBSTRATE_GBUFFER_FORMAT==1 && SubstrateFirstMRT > 0)
	{
		OutEnvironment.SetDefine(TEXT("SUBSTRATE_FIRST_MRT_INDEX"), SubstrateFirstMRT);
	}
}

void FShaderCompileUtilities::AppendGBufferDDCKeyString(const EShaderPlatform Platform, FString& KeyString)
{
	FShaderKeyGenerator KeyGen(KeyString);
	AppendGBufferDDCKey(Platform, KeyGen);
}

void FShaderCompileUtilities::AppendGBufferDDCKey(const EShaderPlatform Platform, FShaderKeyGenerator& KeyGen)
{
	for (uint32 Layout = 0; Layout < GBL_Num; ++Layout)
	{
		FShaderGlobalDefines GlobalDefines = FetchShaderGlobalDefines(Platform, (EGBufferLayout)Layout);
		KeyGen.AppendSeparator();
		KeyGen.AppendBoolInt(GlobalDefines.GBUFFER_HAS_VELOCITY);
		KeyGen.AppendBoolInt(GlobalDefines.GBUFFER_HAS_TANGENT);
		KeyGen.AppendBoolInt(GlobalDefines.ALLOW_STATIC_LIGHTING);
	}
	KeyGen.AppendSeparator();
	KeyGen.Append(GBufferGeneratorVersion);
	KeyGen.AppendDebugText(TEXTVIEW(";\n"));
}

static FGBufferInfo GLastGBufferInfo[SP_NumPlatforms] = {};
static bool GLastGBufferIsValid[SP_NumPlatforms] = {}; // set to all false

void FShaderCompileUtilities::WriteGBufferInfoAutogen(EShaderPlatform TargetPlatform, ERHIFeatureLevel::Type FeatureLevel = ERHIFeatureLevel::SM5)
{
	FGBufferParams DefaultParams = FetchGBufferParamsPipeline(TargetPlatform, GBL_Default);

	FScopeLock MapLock(&GCriticalSection);

	FString AutoGenDirectory = GetAutoGenDirectory(TargetPlatform);
	FString AutogenHeaderFilename = AutoGenDirectory / TEXT("AutogenShaderHeaders.ush");
	FString AutogenHeaderFilenameTemp = AutoGenDirectory / TEXT("AutogenShaderHeaders_temp.ush");

	// auto-generated GBuffer layout is not used by mobile rendering
	if (FeatureLevel <= ERHIFeatureLevel::ES3_1)
	{
		if (GLastGBufferIsValid[TargetPlatform])
		{
			// We've already ensured the autogen file is removed.
			return;
		}

		IPlatformFile& PlatformFile = FPlatformFileManager::Get().GetPlatformFile();
		if (PlatformFile.FileExists(*AutogenHeaderFilename))
		{
			if (PlatformFile.DeleteFile(*AutogenHeaderFilename))
			{
				GLastGBufferIsValid[TargetPlatform] = true;
			}
			else
			{
				UE_LOGF(LogShaderCompilers, Display, "Failed to delete old and now unused shader autogen file: %ls", *AutogenHeaderFilename);
			}
		}
		else
		{
			GLastGBufferIsValid[TargetPlatform] = true;
		}
		return;
	}

	// For now, the logic always calculates the new GBuffer, and if it's the first time, write it, otherwise check it hasn't changed. We are doing this for
	// debugging, and in the near future it will only calculate the GBuffer on the first time only.

	FGBufferInfo DefaultBufferInfo = FetchFullGBufferInfo(DefaultParams);

	if (GLastGBufferIsValid[TargetPlatform])
	{
		const bool bSame = IsGBufferInfoEqual(GLastGBufferInfo[TargetPlatform], DefaultBufferInfo);
		check(bSame);
	}
	else
	{
		GLastGBufferIsValid[TargetPlatform] = true;

		// should cache this properly, and serialize it, but this is a temporary fix.
		GLastGBufferInfo[TargetPlatform] = DefaultBufferInfo;

		FString OutputFileData;
		OutputFileData += TEXT("// Copyright Epic Games, Inc. All Rights Reserved.\n");
		OutputFileData += TEXT("\n");
		OutputFileData += TEXT("#pragma once\n");
		OutputFileData += TEXT("\n");


		OutputFileData += TEXT("#if FEATURE_LEVEL >= FEATURE_LEVEL_SM5\n");
		OutputFileData += TEXT("float SampleDeviceZFromSceneTexturesTempCopy(float2 UV)\n");
		OutputFileData += TEXT("{\n");
		OutputFileData += TEXT("\treturn SceneDepthTexture.SampleLevel(SceneDepthTextureSampler, UV, 0).r;\n");
		OutputFileData += TEXT("}\n");
		OutputFileData += TEXT("#endif\n");
		OutputFileData += TEXT("\n");

		OutputFileData += TEXT("#ifndef GBUFFER_LAYOUT\n");
		OutputFileData += TEXT("#define GBUFFER_LAYOUT 0\n");
		OutputFileData += TEXT("#endif\n");
		OutputFileData += TEXT("\n");

		for (uint32 Layout = 0; Layout < GBL_Num; ++Layout)
		{
			FGBufferParams Params = FetchGBufferParamsPipeline(TargetPlatform, (EGBufferLayout)Layout);
			FGBufferInfo BufferInfo = FetchFullGBufferInfo(Params);

			OutputFileData.Appendf(TEXT("#if GBUFFER_LAYOUT == %u\n\n"), Layout);
			OutputFileData += CreateGBufferEncodeFunction(BufferInfo);

			OutputFileData += TEXT("\n");

			OutputFileData += CreateGBufferDecodeFunctionDirect(BufferInfo);

			OutputFileData += TEXT("\n");
			//OutputFileData += TEXT("#if SHADING_PATH_DEFERRED\n");
			OutputFileData += TEXT("#if FEATURE_LEVEL >= FEATURE_LEVEL_SM5\n");
			OutputFileData += TEXT("\n");

			OutputFileData += CreateGBufferDecodeFunctionVariation(BufferInfo, EGBufferDecodeType::CoordUV, FeatureLevel);
			OutputFileData += TEXT("\n");

			OutputFileData += CreateGBufferDecodeFunctionVariation(BufferInfo, EGBufferDecodeType::CoordUInt, FeatureLevel);

			OutputFileData += TEXT("\n");

			OutputFileData += CreateGBufferDecodeFunctionVariation(BufferInfo, EGBufferDecodeType::SceneTextures, FeatureLevel);
			OutputFileData += TEXT("\n");

			OutputFileData += CreateGBufferDecodeFunctionVariation(BufferInfo, EGBufferDecodeType::SceneTexturesLoad, FeatureLevel);
			OutputFileData += TEXT("\n");

			OutputFileData += TEXT("#endif\n");
			OutputFileData += TEXT("\n");

			OutputFileData += TEXT("#endif\n");
			OutputFileData += TEXT("\n");
		}


		UE_LOGF(LogShaderCompilers, Display, "Compiling shader autogen file: %ls", *AutogenHeaderFilename);

		// If this file already exists, and the text file is the same as what we are planning to write, then don't write it. In that case,
		// we don't need to update it (since it didn't change) and it leaves the timestamp unchanged in case you need to open the file manually
		// for debugging.
		bool bWriteNeeded = true;
		{
			FString PrevFileData;
			bool bFileExists = FFileHelper::LoadFileToString(PrevFileData,*AutogenHeaderFilename);
			if (bFileExists && PrevFileData == OutputFileData)
			{
				UE_LOGF(LogShaderCompilers, Display, "Autogen file is unchanged, skipping write.");
				bWriteNeeded = false;
			}
		}

		if (bWriteNeeded)
		{
			FPaths::MakeStandardFilename(AutogenHeaderFilenameTemp);
			bool bSaveOk = FFileHelper::SaveStringToFile(OutputFileData, *AutogenHeaderFilenameTemp, FFileHelper::EEncodingOptions::ForceAnsi);
			if (!bSaveOk)
			{
				UE_LOGF(LogShaderCompilers, Display, "Failed to save shader autogen file: %ls", *AutogenHeaderFilename);
			}

			bool bDeleteOk = FPlatformFileManager::Get().GetPlatformFile().DeleteFile(*AutogenHeaderFilename);
			if (!bDeleteOk)
			{
				// note that it will fail to delete when the file does not exist, which is acceptable. so in that case
				// write to the log, but don't error out
				UE_LOGF(LogShaderCompilers, Display, "Failed to delete old shader autogen file: %ls", *AutogenHeaderFilename);
			}

			bool bRenameOk = FPlatformFileManager::Get().GetPlatformFile().MoveFile(*AutogenHeaderFilename, *AutogenHeaderFilenameTemp);
			if (!bRenameOk)
			{
				UE_LOGF(LogShaderCompilers, Display, "Failed to rename shader autogen file: %ls", *AutogenHeaderFilename);
			}

			UE_LOGF(LogShaderCompilers, Display, "Shader autogen file written: %ls", *AutogenHeaderFilename);
		}
	}
}

void FShaderCompileUtilities::GenerateBrdfHeaders(EShaderPlatform TargetPlatform)
{
	ERHIFeatureLevel::Type FeatureLevel = GetMaxSupportedFeatureLevel(TargetPlatform);

	// Writes the GBuffer format .ush file if it's out of date.
	WriteGBufferInfoAutogen(TargetPlatform, FeatureLevel);
}

void FShaderCompileUtilities::GenerateBrdfHeaders(const FName& ShaderFormat)
{
	const EShaderPlatform ShaderPlatform = ShaderFormatToLegacyShaderPlatform(ShaderFormat);
	FShaderCompileUtilities::GenerateBrdfHeaders(ShaderPlatform);
}

EGBufferLayout FShaderCompileUtilities::FetchGBufferLayout(const FShaderCompilerEnvironment& Environment)
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	const uint32 Layout = Environment.Definitions->GetIntegerValue(TEXT("GBUFFER_LAYOUT"));
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	if (Layout >= GBL_Num)
	{
		return GBL_Default;
	}
	return (EGBufferLayout)Layout;
}

FGBufferParams FShaderCompileUtilities::FetchGBufferParamsPipeline(EShaderPlatform Platform, EGBufferLayout Layout)
{
	FGBufferParams Ret = {};
	Ret.ShaderPlatform = Platform;

#if WITH_EDITOR
	Ret.bHasPrecShadowFactor = IsStaticLightingAllowed();
	Ret.bHasVelocity = (IsUsingBasePassVelocity(Platform) || Layout == GBL_ForceVelocity) ? 1 : 0;
	Ret.bHasTangent = false;//BasePassCanOutputTangent(TargetPlatform) ? 1 : 0;

	{
		Ret.bUsesVelocityDepth = NeedsVelocityDepth(Platform);
	}

	{
		static IConsoleVariable* CVar = IConsoleManager::Get().FindConsoleVariable(TEXT("r.GBufferFormat"));
		Ret.LegacyFormatIndex = CVar->GetInt();
	}
#else
	// this function should never be called in a non-editor build
	check(0);
#endif


	return Ret;
}

#endif  // WITH_EDITOR

FGBufferParams FShaderCompileUtilities::FetchGBufferParamsRuntime(EShaderPlatform Platform, EGBufferLayout Layout)
{
	// This code should match TBasePassPS

	FGBufferParams Ret = {};
	Ret.ShaderPlatform = Platform;

	Ret.bHasPrecShadowFactor = IsStaticLightingAllowed();

	Ret.bHasVelocity = (IsUsingBasePassVelocity(Platform) || Layout == GBL_ForceVelocity) ? 1 : 0;
	Ret.bHasTangent = false;//BasePassCanOutputTangent(ShaderPlatform) ? 1 : 0;

	Ret.bUsesVelocityDepth = NeedsVelocityDepth(Platform);

	static const auto CVarFormat = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.GBufferFormat"));
	Ret.LegacyFormatIndex = CVarFormat->GetValueOnAnyThread();

	// This should match with SINGLE_LAYER_WATER_SEPARATED_MAIN_LIGHT
	Ret.bHasSingleLayerWaterSeparatedMainLight = IsWaterSeparateMainDirLightEnabled(Platform);

	return Ret;
}
