// Copyright Epic Games, Inc. All Rights Reserved.

#include "ShadowMap.h"
#include "Engine/Level.h"
#include "Engine/MapBuildDataRegistry.h"
#include "StaticLightingBuildContext.h"
#include "Components/LightComponent.h"

#include "Engine/World.h"
#include "TextureLayout.h"
#include "Interfaces/ITargetPlatform.h"
#include "Engine/ShadowMapTexture2D.h"
#include "Components/InstancedStaticMeshComponent.h"
#include "LightMap.h"
#include "Misc/FeedbackContext.h"
#include "Misc/QueuedThreadPool.h"
#include "Modules/ModuleManager.h"
#include "GameFramework/WorldSettings.h"
#include "Templates/Greater.h"

#if WITH_EDITOR

	#include "Modules/ModuleManager.h"
	#include "TextureCompressorModule.h"

	// NOTE: We're only counting the top-level mip-map for the following variables.
	/** Total number of texels allocated for all shadowmap textures. */
	ENGINE_API uint64 GNumShadowmapTotalTexels = 0;
	/** Number of shadowmap textures generated. */
	ENGINE_API int32 GNumShadowmapTextures = 0;
	/** Total number of mapped texels. */
	ENGINE_API uint64 GNumShadowmapMappedTexels = 0;
	/** Total number of unmapped texels. */
	ENGINE_API uint64 GNumShadowmapUnmappedTexels = 0;
	/** Whether to allow cropping of unmapped borders in lightmaps and shadowmaps. Controlled by BaseEngine.ini setting. */
	extern ENGINE_API bool GAllowLightmapCropping;
	/** Total shadowmap texture memory size (in bytes), including GShadowmapTotalStreamingSize. */
	ENGINE_API uint64 GShadowmapTotalSize = 0;
	/** Total texture memory size for streaming shadowmaps. */
	ENGINE_API uint64 GShadowmapTotalStreamingSize = 0;

	/** Whether to allow lighting builds to generate streaming lightmaps. */
	extern ENGINE_API bool GAllowStreamingLightmaps;
	extern ENGINE_API float GMaxLightmapRadius;
#endif


UShadowMapTexture2D::UShadowMapTexture2D(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	LODGroup = TEXTUREGROUP_Shadowmap;
}

void FShadowMap::Serialize(FArchive& Ar)
{
	Ar << LightGuids;
}

void FShadowMap::Cleanup()
{
	BeginCleanup(this);
}

#if WITH_EDITOR

struct FShadowMapAllocation
{
	TRefCountPtr<FShadowMap2D>	ShadowMap;

	UObject*					Primitive;
	UMapBuildDataRegistry* Registry;
	FGuid MapBuildDataId;
	int32						InstanceIndex;

	/** Upper-left X-coordinate in the texture atlas. */
	int32						OffsetX;
	/** Upper-left Y-coordinate in the texture atlas. */
	int32						OffsetY;
	/** Total number of texels along the X-axis. */
	int32						TotalSizeX;
	/** Total number of texels along the Y-axis. */
	int32						TotalSizeY;
	/** The rectangle of mapped texels within this mapping that is placed in the texture atlas. */
	FIntRect					MappedRect;
	ELightMapPaddingType		PaddingType;

	TMap<ULightComponent*, TArray<FQuantizedSignedDistanceFieldShadowSample> > ShadowMapData;

	FShadowMapAllocation()
	{
		PaddingType = GAllowLightmapPadding ? LMPT_NormalPadding : LMPT_NoPadding;
		MappedRect.Min.X = 0;
		MappedRect.Min.Y = 0;
		MappedRect.Max.X = 0;
		MappedRect.Max.Y = 0;
		Primitive = nullptr;
		Registry = NULL;
		InstanceIndex = INDEX_NONE;
	}

	// Called after the shadowmap is encoded
	void PostEncode()
	{
		if (InstanceIndex >= 0 && Registry)
		{
			FMeshMapBuildData* MeshBuildData = Registry->GetMeshBuildDataDuringBuild(MapBuildDataId);
			check(MeshBuildData);

			UInstancedStaticMeshComponent* Component = CastChecked<UInstancedStaticMeshComponent>(Primitive);

			// Instances may have been removed since LM allocation.
			// Instances may have also been shuffled from removes. We do not handle this case.
			if( InstanceIndex < MeshBuildData->PerInstanceLightmapData.Num() )
			{
				// TODO: We currently only support one LOD of static lighting in foliage
				// Need to create per-LOD instance data to fix that
				MeshBuildData->PerInstanceLightmapData[InstanceIndex].ShadowmapUVBias = FVector2f(ShadowMap->GetCoordinateBias());
				Component->SetBakedLightingDataChanged(InstanceIndex);
			}
		}
	}
};

struct FShadowMapAllocationGroup
{
	FShadowMapAllocationGroup()
		: TextureOuter(nullptr)
		, Bounds(ForceInit)
		, ShadowmapFlags(SMF_None)
		, TotalTexels(0)
	{
	}

	FShadowMapAllocationGroup(FShadowMapAllocationGroup&&) = default;
	FShadowMapAllocationGroup& operator=(FShadowMapAllocationGroup&&) = default;

	FShadowMapAllocationGroup(const FShadowMapAllocationGroup&) = delete; // non-copy-able
	FShadowMapAllocationGroup& operator=(const FShadowMapAllocationGroup&) = delete;

	TArray<TUniquePtr<FShadowMapAllocation>, TInlineAllocator<1>> Allocations;

	UObject*			TextureOuter;

	/** Bounds of the primitive that the mapping is applied to. */
	FBoxSphereBounds	Bounds;
	/** Bit-field with shadowmap flags. */
	EShadowMapFlags		ShadowmapFlags;

	int32				TotalTexels;
};

/** Whether to try to pack procbuilding lightmaps/shadowmaps into the same texture. */
extern bool GGroupComponentLightmaps;

struct FShadowMapPendingTexture : FTextureLayout
{
	TArray<TUniquePtr<FShadowMapAllocation>> Allocations;

	UObject* Outer;

	/** Bounds for all shadowmaps in this texture. */
	FBoxSphereBounds	Bounds;
	/** Bit-field with shadowmap flags that are shared among all shadowmaps in this texture. */
	EShadowMapFlags		ShadowmapFlags;

	// Optimization to quickly test if a new allocation won't fit
	// Primarily of benefit to instanced mesh shadowmaps
	int32				UnallocatedTexels;

	// have we created the uobjects (in this case the Texture)
	bool bCreatedUObjects;
	// shadowmap texture
	UShadowMapTexture2D* ShadowMapTexture;
	volatile bool		bFinishedEncoding;
	bool				bHasRunPostEncode;
	/**
	 * Minimal initialization constructor.
	 */
	FShadowMapPendingTexture(uint32 InSizeX,uint32 InSizeY)
		: FTextureLayout(4, 4, InSizeX, InSizeY, /* PowerOfTwo */ true, /* Aspect */ ETextureLayoutAspectRatio::None, /* AlignByFour */ true) // Min size is 4x4 in case of block compression.
		, Bounds(FBox(ForceInit))
		, ShadowmapFlags(SMF_None)
		, UnallocatedTexels(InSizeX * InSizeY)
		, bCreatedUObjects( false)
		, ShadowMapTexture(nullptr)
		, bFinishedEncoding(false)
	{
	}

	bool AddElement(FShadowMapAllocationGroup& AllocationGroup, const bool bForceIntoThisTexture = false);

	/**
	 * Begin encoding the textures
	 *
	 * @param	InWorld	World in which the textures exist
	 */
	void StartEncoding(const FStaticLightingBuildContext* LightingContext, ITextureCompressorModule* Compressor);

	/**
	 * Create UObjects required in the encoding step, this is so we can multithread teh encode step
	 */
	void CreateUObjects();

	/**
	 * After multithreaded encode t
	 */
	void PostEncode();

	bool IsFinishedEncoding() const { return bFinishedEncoding; }

	void FinishCachingTextures(UWorld* InWorld);
};

bool FShadowMapPendingTexture::AddElement(FShadowMapAllocationGroup& AllocationGroup, const bool bForceIntoThisTexture)
{
	if (!bForceIntoThisTexture)
	{
		// Don't pack shadowmaps from different packages into the same texture.
		if (Outer != AllocationGroup.TextureOuter)
		{
			return false;
		}
	}

	// This is a rough test, passing it doesn't guarantee it'll fit
	// But failing it does guarantee that it _won't_ fit
	if (UnallocatedTexels < AllocationGroup.TotalTexels)
	{
		return false;
	}

	const bool bEmptyTexture = Allocations.Num() == 0;
	const FBoxSphereBounds NewBounds = bEmptyTexture ? AllocationGroup.Bounds : Bounds + AllocationGroup.Bounds;

	if (!bEmptyTexture && !bForceIntoThisTexture)
	{
		// Don't mix streaming shadowmaps with non-streaming shadowmaps.
		if ((ShadowmapFlags & LMF_Streamed) != (AllocationGroup.ShadowmapFlags & LMF_Streamed))
		{
			return false;
		}

		// Is this a streaming shadowmap?
		if (ShadowmapFlags & LMF_Streamed)
		{
			bool bPerformDistanceCheck = true;

			// Don't pack together shadowmaps that are too far apart
			if (bPerformDistanceCheck && NewBounds.SphereRadius > GMaxLightmapRadius && NewBounds.SphereRadius > (Bounds.SphereRadius + UE_SMALL_NUMBER))
			{
				return false;
			}
		}
	}

	int32 NewUnallocatedTexels = UnallocatedTexels;

	int32 iAllocation = 0;
	for (; iAllocation < AllocationGroup.Allocations.Num(); ++iAllocation)
	{
		auto& Allocation = AllocationGroup.Allocations[iAllocation];
		uint32 BaseX, BaseY;
		const uint32 AllocationSizeX = Allocation->MappedRect.Width();
		const uint32 AllocationSizeY = Allocation->MappedRect.Height();
		if (FTextureLayout::AddElement(BaseX, BaseY, AllocationSizeX, AllocationSizeY))
		{
			Allocation->OffsetX = BaseX;
			Allocation->OffsetY = BaseY;

			// Assumes bAlignByFour
			NewUnallocatedTexels -= ((AllocationSizeX + 3) & ~3) * ((AllocationSizeY + 3) & ~3);
		}
		else
		{
			// failed to add all elements to the texture
			break;
		}
	}
	if (iAllocation < AllocationGroup.Allocations.Num())
	{
		// failed to add all elements to the texture
		// remove the ones added so far to restore our original state
		for (--iAllocation; iAllocation >= 0; --iAllocation)
		{
			auto& Allocation = AllocationGroup.Allocations[iAllocation];
			const uint32 AllocationSizeX = Allocation->MappedRect.Width();
			const uint32 AllocationSizeY = Allocation->MappedRect.Height();
			verify(FTextureLayout::RemoveElement(Allocation->OffsetX, Allocation->OffsetY, AllocationSizeX, AllocationSizeY));
		}
		return false;
	}

	Bounds = NewBounds;
	UnallocatedTexels = NewUnallocatedTexels;

	return true;
}

void FShadowMapPendingTexture::CreateUObjects()
{
	if (bCreatedUObjects == false)
	{
		check(IsInGameThread());
		ShadowMapTexture = NewObject<UShadowMapTexture2D>(Outer);
	}
	bCreatedUObjects = true;
}

void FShadowMapPendingTexture::StartEncoding(const FStaticLightingBuildContext* LightingContext, ITextureCompressorModule* Compressor)
{
	FTaskTagScope Scope(ETaskTag::EParallelGameThread);

	// Create the shadow-map texture.
	CreateUObjects();

	auto Texture = ShadowMapTexture;

	Texture->Filter			= GUseBilinearLightmaps ? TF_Default : TF_Nearest;
	// Signed distance field textures get stored in linear space, since they need more precision near .5.
	Texture->SRGB			= false;
	Texture->LODGroup		= TEXTUREGROUP_Shadowmap;
	Texture->ShadowmapFlags	= ShadowmapFlags;

	{
		// Create the uncompressed top mip-level.
		TArray< TArray<FFourDistanceFieldSamples> > MipData;
		const int32 NumChannelsUsed = FShadowMap2D::EncodeSingleTexture(LightingContext, *this, Texture, MipData);

		Texture->Source.Init2DWithMipChain(GetSizeX(), GetSizeY(), NumChannelsUsed == 1 ? TSF_G8 : TSF_BGRA8);
		Texture->MipGenSettings = TMGS_LeaveExistingMips;

		// Uncompressed G8 or BGRA8 platform data :
		Texture->CompressionSettings = NumChannelsUsed == 1 ? TC_Grayscale : TC_Default;
		Texture->CompressionNone = true;
		
		// outer MipLock
		FTextureSource::FMipLock Mip0Lock(FTextureSource::ELockState::ReadWrite,&Texture->Source,0);

		// Copy the mip-map data into the UShadowMapTexture2D's mip-map array.
		for(int32 MipIndex = 0;MipIndex < MipData.Num();MipIndex++)
		{
			uint8* DestMipData = (uint8*)Texture->Source.LockMip(MipIndex);
			uint32 MipSizeX = FMath::Max<uint32>(1, GetSizeX() >> MipIndex);
			uint32 MipSizeY = FMath::Max<uint32>(1, GetSizeY() >> MipIndex);

			for(uint32 Y = 0;Y < MipSizeY;Y++)
			{
				for(uint32 X = 0;X < MipSizeX;X++)
				{
					const FFourDistanceFieldSamples& SourceSample = MipData[MipIndex][Y * MipSizeX + X];

					if (NumChannelsUsed == 1)
					{
						DestMipData[Y * MipSizeX + X] = SourceSample.Samples[0].Distance;
					}
					else
					{
						// 4 distances in RGBA channels
						((FColor*)DestMipData)[Y * MipSizeX + X] = FColor(SourceSample.Samples[0].Distance, SourceSample.Samples[1].Distance, SourceSample.Samples[2].Distance, SourceSample.Samples[3].Distance);
					}
				}
			}

			Texture->Source.UnlockMip(MipIndex);
		}
	}

	// Update the texture resource.
	// @todo Oodle : manually calling CachePlatformData is not standard; not sure if this is a bug or was intentional for some reason
	//		more standard thing should be to just call PostEditChange() here and let UpdateResource do its thing.
	Texture->CachePlatformData(true, true, false, Compressor);

	bFinishedEncoding = true;
}

void FShadowMapPendingTexture::PostEncode()
{
	check(bFinishedEncoding);

	if (!bHasRunPostEncode)
	{
		check(IsInGameThread());
		auto Texture = ShadowMapTexture;

		Texture->CachePlatformData(true, true);
		bHasRunPostEncode = true;
	}
}


void FShadowMapPendingTexture::FinishCachingTextures(UWorld* InWorld)
{
	check(IsInGameThread());
	auto Texture = ShadowMapTexture;

	Texture->FinishCachePlatformData();
	Texture->UpdateResource();

	// Update stats.
	int32 TextureSize = Texture->CalcTextureMemorySizeEnum(TMC_AllMips);
	GNumShadowmapTotalTexels += GetSizeX() * GetSizeY();
	GNumShadowmapTextures++;
	GShadowmapTotalSize += TextureSize;
	GShadowmapTotalStreamingSize += (ShadowmapFlags & SMF_Streamed) ? TextureSize : 0;

	UPackage* TexturePackage = Texture->GetOutermost();

	for (int32 LevelIndex = 0; TexturePackage && LevelIndex < InWorld->GetNumLevels(); LevelIndex++)
	{
		ULevel* Level = InWorld->GetLevel(LevelIndex);
		UPackage* LevelPackage = Level->GetOutermost();
		if (TexturePackage == LevelPackage)
		{
			Level->ShadowmapTotalSize += float(TextureSize) / 1024.0f;
			break;
		}
	}
}

static TArray<FShadowMapAllocationGroup> PendingShadowMaps;
static uint32 PendingShadowMapSize;
/** If true, update the status when encoding light maps */
bool FShadowMap2D::bUpdateStatus = true;

#endif 

TRefCountPtr<FShadowMap2D> FShadowMap2D::AllocateShadowMap(
	UObject* LightMapOuter, 
	const TMap<ULightComponent*,FShadowMapData2D*>& ShadowMapData,
	const FBoxSphereBounds& Bounds, 
	ELightMapPaddingType InPaddingType,
	EShadowMapFlags InShadowmapFlags)
{
#if WITH_EDITOR
	check(ShadowMapData.Num() > 0);

	FShadowMapAllocationGroup AllocationGroup;
	AllocationGroup.TextureOuter = LightMapOuter;
	AllocationGroup.ShadowmapFlags = InShadowmapFlags;
	AllocationGroup.Bounds = Bounds;
	if (!GAllowStreamingLightmaps)
	{
		AllocationGroup.ShadowmapFlags = EShadowMapFlags(AllocationGroup.ShadowmapFlags & ~SMF_Streamed);
	}

	// Create a new shadow-map.
	TRefCountPtr<FShadowMap2D> ShadowMap = TRefCountPtr<FShadowMap2D>(new FShadowMap2D(ShadowMapData));

	// Calculate Shadowmap size
	int32 SizeX = -1;
	int32 SizeY = -1;
	int32 LightIndex = 0;
	for (const auto& ShadowDataPair : ShadowMapData)
	{
		if (LightIndex == 0)
		{
			SizeX = ShadowDataPair.Value->GetSizeX();
			SizeY = ShadowDataPair.Value->GetSizeY();
		}
		else
		{
			check(SizeX == ShadowDataPair.Value->GetSizeX() && SizeY == ShadowDataPair.Value->GetSizeY());
		}

		LightIndex++;
	}
	check(SizeX != -1 && SizeY != -1);

	// Add a pending allocation for this shadow-map.
	TUniquePtr<FShadowMapAllocation> Allocation = MakeUnique<FShadowMapAllocation>();
	Allocation->PaddingType = InPaddingType;
	Allocation->ShadowMap = ShadowMap;
	Allocation->TotalSizeX = SizeX;
	Allocation->TotalSizeY = SizeY;
	Allocation->MappedRect = FIntRect(0, 0, SizeX, SizeY);
	Allocation->PaddingType = InPaddingType;

	for (const auto& ShadowDataPair : ShadowMapData)
	{
		const FShadowMapData2D* RawData = ShadowDataPair.Value;
		TArray<FQuantizedSignedDistanceFieldShadowSample>& DistanceFieldShadowData = Allocation->ShadowMapData.Add(ShadowDataPair.Key, TArray<FQuantizedSignedDistanceFieldShadowSample>());

		switch (RawData->GetType())
		{
		case FShadowMapData2D::SHADOW_SIGNED_DISTANCE_FIELD_DATA:
		case FShadowMapData2D::SHADOW_SIGNED_DISTANCE_FIELD_DATA_QUANTIZED:
			// If the data is already quantized, this will just copy the data
			RawData->Quantize(DistanceFieldShadowData);
			break;
		default:
			check(0);
		}

		delete RawData;

		// Track the size of pending light-maps.
		PendingShadowMapSize += Allocation->TotalSizeX * Allocation->TotalSizeY;
	}

	// Assumes bAlignByFour
	AllocationGroup.TotalTexels += ((Allocation->MappedRect.Width() + 3) & ~3) * ((Allocation->MappedRect.Height() + 3) & ~3);

	AllocationGroup.Allocations.Add(MoveTemp(Allocation));

	PendingShadowMaps.Add(MoveTemp(AllocationGroup));

	return ShadowMap;
#else
	return nullptr;
#endif
}

FShadowMap2D::FShadowMap2D() :
	Texture(nullptr),
	CoordinateScale(FVector2D(0, 0)),
	CoordinateBias(FVector2D(0, 0))
{
	for (int Channel = 0; Channel < UE_ARRAY_COUNT(bChannelValid); Channel++)
	{
		bChannelValid[Channel] = false;
	}
}

FShadowMap2D::FShadowMap2D(const TMap<ULightComponent*,FShadowMapData2D*>& ShadowMapData) :
	Texture(nullptr),
	CoordinateScale(FVector2D(0, 0)),
	CoordinateBias(FVector2D(0, 0))
{
	for (int Channel = 0; Channel < UE_ARRAY_COUNT(bChannelValid); Channel++)
	{
		bChannelValid[Channel] = false;
	}

	for (TMap<ULightComponent*, FShadowMapData2D*>::TConstIterator It(ShadowMapData); It; ++It)
	{
		LightGuids.Add(It.Key()->LightGuid);
	}
}

FShadowMap2D::FShadowMap2D(TArray<FGuid> LightGuids)
	: FShadowMap(MoveTemp(LightGuids))
	, Texture(nullptr)
	, CoordinateScale(FVector2D(0, 0))
	, CoordinateBias(FVector2D(0, 0))
{
	for (int Channel = 0; Channel < UE_ARRAY_COUNT(bChannelValid); Channel++)
	{
		bChannelValid[Channel] = false;
	}
}

UTexture2D* FShadowMap2D::GetTexture() { check(IsValid()); return Texture; }
const UTexture2D* FShadowMap2D::GetTexture() const { check(IsValid()); return Texture; }

void FShadowMap2D::AddReferencedObjects(FReferenceCollector& Collector)
{
	Collector.AddReferencedObject(Texture);
}

void FShadowMap2D::Serialize(FArchive& Ar)
{
	FShadowMap::Serialize(Ar);
	
	if( Ar.IsCooking() && !Ar.CookingTarget()->SupportsFeature(ETargetPlatformFeatures::DistanceFieldShadows) )
	{
		UShadowMapTexture2D* Dummy = NULL;
		Ar << Dummy;
	}
	else
	{
		Ar << Texture;
	}

	Ar << CoordinateScale << CoordinateBias;

	for (int Channel = 0; Channel < UE_ARRAY_COUNT(bChannelValid); Channel++)
	{
		Ar << bChannelValid[Channel];
	}

	if (Ar.UEVer() >= VER_UE4_STATIC_SHADOWMAP_PENUMBRA_SIZE)
	{
		Ar << InvUniformPenumbraSize;
	}
	else if (Ar.IsLoading())
	{
		const float LegacyValue = 1.0f / .05f;
		InvUniformPenumbraSize = FVector4f(LegacyValue, LegacyValue, LegacyValue, LegacyValue);
	}
}

FShadowMapInteraction FShadowMap2D::GetInteraction() const
{
	if (Texture)
	{
		return FShadowMapInteraction::Texture(Texture, CoordinateScale, CoordinateBias, bChannelValid, InvUniformPenumbraSize);
	}
	return FShadowMapInteraction::None();
}


TRefCountPtr<FShadowMap2D> FShadowMap2D::AllocateInstancedShadowMap(UObject* LightMapOuter, UInstancedStaticMeshComponent* Component, TArray<TMap<ULightComponent*, TUniquePtr<FShadowMapData2D>>>&& InstancedShadowMapData,
	UMapBuildDataRegistry* Registry, FGuid MapBuildDataId, const FBoxSphereBounds& Bounds, ELightMapPaddingType InPaddingType, EShadowMapFlags InShadowmapFlags)
{
#if WITH_EDITOR
	check(InstancedShadowMapData.Num() > 0);

	// Verify all instance shadowmaps are the same size, and build complete list of shadow lights
	int32 SizeX = -1;
	int32 SizeY = -1;
	TSet<ULightComponent*> AllLights;
	for (auto& ShadowMapData : InstancedShadowMapData)
	{
		for (const auto& ShadowDataPair : ShadowMapData)
		{
			if (SizeX == -1)
			{
				SizeX = ShadowDataPair.Value->GetSizeX();
				SizeY = ShadowDataPair.Value->GetSizeY();
			}
			else
			{
				check(ShadowDataPair.Value->GetSizeX() == SizeX);
				check(ShadowDataPair.Value->GetSizeY() == SizeY);
			}
			AllLights.Add(ShadowDataPair.Key);
		}
	}

	check(SizeX != -1 && SizeY != -1); // No valid shadowmaps

	TArray<FGuid> LightGuids;
	LightGuids.Reserve(AllLights.Num());
	for (ULightComponent* Light : AllLights)
	{
		LightGuids.Add(Light->LightGuid);
	}

	// Unify all the shadow map data to contain the same lights in the same order
	for (auto& ShadowMapData : InstancedShadowMapData)
	{
		for (ULightComponent* Light : AllLights)
		{
			if (!ShadowMapData.Contains(Light))
			{
				ShadowMapData.Add(Light, MakeUnique<FQuantizedShadowSignedDistanceFieldData2D>(SizeX, SizeY));
			}
		}
	}

	FShadowMapAllocationGroup AllocationGroup;
	AllocationGroup.TextureOuter = LightMapOuter;
	AllocationGroup.ShadowmapFlags = InShadowmapFlags;
	AllocationGroup.Bounds = Bounds;
	if (!GAllowStreamingLightmaps)
	{
		AllocationGroup.ShadowmapFlags = EShadowMapFlags(AllocationGroup.ShadowmapFlags & ~SMF_Streamed);
	}

	TRefCountPtr<FShadowMap2D> BaseShadowmap = nullptr;

	for (int32 InstanceIndex = 0; InstanceIndex < InstancedShadowMapData.Num(); ++InstanceIndex)
	{
		auto& ShadowMapData = InstancedShadowMapData[InstanceIndex];
		check(ShadowMapData.Num() > 0);

		// Create a new shadow-map.
		TRefCountPtr<FShadowMap2D> ShadowMap = TRefCountPtr<FShadowMap2D>(new FShadowMap2D(LightGuids));

		if (InstanceIndex == 0)
		{
			BaseShadowmap = ShadowMap;
		}

		// Add a pending allocation for this shadow-map.
		TUniquePtr<FShadowMapAllocation> Allocation = MakeUnique<FShadowMapAllocation>();
		Allocation->PaddingType = InPaddingType;
		Allocation->ShadowMap = MoveTemp(ShadowMap);
		Allocation->TotalSizeX = SizeX;
		Allocation->TotalSizeY = SizeY;
		Allocation->MappedRect = FIntRect(0, 0, SizeX, SizeY);
		Allocation->Primitive = Component;
		Allocation->Registry = Registry;
		Allocation->MapBuildDataId = MapBuildDataId;
		Allocation->InstanceIndex = InstanceIndex;

		for (auto& ShadowDataPair : ShadowMapData)
		{
			auto& RawData = ShadowDataPair.Value;
			auto& DistanceFieldShadowData = Allocation->ShadowMapData.Add(ShadowDataPair.Key, TArray<FQuantizedSignedDistanceFieldShadowSample>());

			switch (RawData->GetType())
			{
			case FShadowMapData2D::SHADOW_SIGNED_DISTANCE_FIELD_DATA:
			case FShadowMapData2D::SHADOW_SIGNED_DISTANCE_FIELD_DATA_QUANTIZED:
				// If the data is already quantized, this will just copy the data
				RawData->Quantize(DistanceFieldShadowData);
				break;
			default:
				check(0);
			}

			RawData.Reset();

			// Track the size of pending light-maps.
			PendingShadowMapSize += Allocation->TotalSizeX * Allocation->TotalSizeY;
		}

		// Assumes bAlignByFour
		AllocationGroup.TotalTexels += ((Allocation->MappedRect.Width() + 3) & ~3) * ((Allocation->MappedRect.Height() + 3) & ~3);

		AllocationGroup.Allocations.Add(MoveTemp(Allocation));
	}

	PendingShadowMaps.Add(MoveTemp(AllocationGroup));

	return BaseShadowmap;
#else
	return nullptr;
#endif
}


#if WITH_EDITOR


/**
 * Executes all pending shadow-map encoding requests.
 * @param	InWorld				World in which the textures exist
 * @param	bLightingSuccessful	Whether the lighting build was successful or not.
 */
void FShadowMap2D::EncodeTextures(const FStaticLightingBuildContext* LightingContext, bool bLightingSuccessful, bool bMultithreadedEncode)
{
	if ( bLightingSuccessful )
	{
		GWarn->BeginSlowTask( NSLOCTEXT("ShadowMap2D", "BeginEncodingShadowMapsTask", "Encoding shadow-maps"), false );
		const int32 PackedLightAndShadowMapTextureSize = LightingContext->World->GetWorldSettings()->PackedLightAndShadowMapTextureSize;

		ITextureCompressorModule* TextureCompressorModule = &FModuleManager::LoadModuleChecked<ITextureCompressorModule>(TEXTURE_COMPRESSOR_MODULENAME);

		// Reset the pending shadow-map size.
		PendingShadowMapSize = 0;

		Algo::SortBy(PendingShadowMaps, &FShadowMapAllocationGroup::TotalTexels, TGreater<>());

		// Allocate texture space for each shadow-map.
		TIndirectArray<FShadowMapPendingTexture> PendingTextures;

		for (FShadowMapAllocationGroup& PendingGroup : PendingShadowMaps)
		{
			if (!ensure(PendingGroup.Allocations.Num() >= 1))
			{
				continue;
			}

			int32 MaxWidth = 0;
			int32 MaxHeight = 0;
			for (auto& Allocation : PendingGroup.Allocations)
			{
				MaxWidth = FMath::Max(MaxWidth, Allocation->MappedRect.Width());
				MaxHeight = FMath::Max(MaxHeight, Allocation->MappedRect.Height());
			}

			// Assume bAlignByFour (see FTextureLayout::AddElement)
			MaxWidth = (MaxWidth + 3) & ~3;
			MaxHeight = (MaxHeight + 3) & ~3;

			FShadowMapPendingTexture* Texture = nullptr;

			// Find an existing texture which the shadow-map can be stored in.
			// Shadowmaps will always be 4-pixel aligned...
			for (FShadowMapPendingTexture& ExistingTexture : PendingTextures)
			{
				if (ExistingTexture.AddElement(PendingGroup))
				{
					Texture = &ExistingTexture;
					break;
				}
			}

			if (!Texture)
			{
				int32 NewTextureSizeX = PackedLightAndShadowMapTextureSize;
				int32 NewTextureSizeY = PackedLightAndShadowMapTextureSize;

				// Assumes identically-sized allocations, fit into the smallest square
				const int32 AllocationCountX = FMath::CeilToInt(FMath::Sqrt(static_cast<float>(FMath::DivideAndRoundUp(PendingGroup.Allocations.Num() * MaxHeight, MaxWidth))));
				const int32 AllocationCountY = FMath::DivideAndRoundUp(PendingGroup.Allocations.Num(), AllocationCountX);
				const int32 AllocationSizeX = AllocationCountX * MaxWidth;
				const int32 AllocationSizeY = AllocationCountY * MaxHeight;

				if (AllocationSizeX > NewTextureSizeX || AllocationSizeY > NewTextureSizeY)
				{
					NewTextureSizeX = FMath::RoundUpToPowerOfTwo(AllocationSizeX);
					NewTextureSizeY = FMath::RoundUpToPowerOfTwo(AllocationSizeY);
				}

				// If there is no existing appropriate texture, create a new one.
				Texture = new FShadowMapPendingTexture(NewTextureSizeX, NewTextureSizeY);
				PendingTextures.Add(Texture);
				Texture->Outer = PendingGroup.TextureOuter;
				Texture->Bounds = PendingGroup.Bounds;
				Texture->ShadowmapFlags = PendingGroup.ShadowmapFlags;
				verify(Texture->AddElement(PendingGroup));
			}

			// Give the texture ownership of the allocations
			for (auto& Allocation : PendingGroup.Allocations)
			{
				Texture->Allocations.Add(MoveTemp(Allocation));
			}
		}
		PendingShadowMaps.Empty();

		if (bMultithreadedEncode)
		{
			FThreadSafeCounter Counter(PendingTextures.Num());
			// Encode all the pending textures.
			TArray<FAsyncEncode<FShadowMapPendingTexture>> AsyncEncodeTasks;
			AsyncEncodeTasks.Empty(PendingTextures.Num());
			for (auto& PendingTexture : PendingTextures)
			{
				PendingTexture.CreateUObjects();
				auto AsyncEncodeTask = new (AsyncEncodeTasks)FAsyncEncode<FShadowMapPendingTexture>(&PendingTexture, LightingContext, Counter, TextureCompressorModule);
				GThreadPool->AddQueuedWork(AsyncEncodeTask);
			}

			while (Counter.GetValue() > 0)
			{
				GWarn->UpdateProgress(Counter.GetValue(), PendingTextures.Num());
				FPlatformProcess::Sleep(0.0001f);
			}
		}
		else
		{
			// Encode all the pending textures.
			for (int32 TextureIndex = 0; TextureIndex < PendingTextures.Num(); TextureIndex++)
			{
				FShadowMapPendingTexture& PendingTexture = PendingTextures[TextureIndex];
				PendingTexture.StartEncoding(LightingContext, TextureCompressorModule);
			}
		}

		bool bHasFinishedPostEncode = false;
		while (bHasFinishedPostEncode == false)
		{
			bHasFinishedPostEncode = true;
			for (int32 TextureIndex = 0; TextureIndex < PendingTextures.Num(); TextureIndex++)
			{
				FShadowMapPendingTexture& PendingTexture = PendingTextures[TextureIndex];

				if (PendingTexture.IsFinishedEncoding())
				{
					PendingTexture.PostEncode();
				}
				else
				{
					bHasFinishedPostEncode = false;
					break;
				}
			}
		}

		for (int32 TextureIndex = 0; TextureIndex < PendingTextures.Num(); TextureIndex++)
		{
			FShadowMapPendingTexture& PendingTexture = PendingTextures[TextureIndex];
			PendingTexture.FinishCachingTextures(LightingContext->World);
			if (bUpdateStatus && ((TextureIndex % 20) == 0))
			{
				GWarn->UpdateProgress(TextureIndex, PendingTextures.Num());
			}
		}
		PendingTextures.Empty();

		GWarn->EndSlowTask();
	}
	else
	{
		PendingShadowMaps.Empty();
	}
}

int32 FShadowMap2D::EncodeSingleTexture(const FStaticLightingBuildContext* LightingContext, FShadowMapPendingTexture& PendingTexture, UShadowMapTexture2D* Texture, TArray< TArray<FFourDistanceFieldSamples> >& MipData)
{
	TArray<FFourDistanceFieldSamples>* TopMipData = new(MipData) TArray<FFourDistanceFieldSamples>();
	TopMipData->Empty(PendingTexture.GetSizeX() * PendingTexture.GetSizeY());
	TopMipData->AddZeroed(PendingTexture.GetSizeX() * PendingTexture.GetSizeY());
	int32 TextureSizeX = PendingTexture.GetSizeX();
	int32 TextureSizeY = PendingTexture.GetSizeY();
	int32 MaxChannelsUsed = 0;

	for (int32 AllocationIndex = 0;AllocationIndex < PendingTexture.Allocations.Num();AllocationIndex++)
	{
		FShadowMapAllocation& Allocation = *PendingTexture.Allocations[AllocationIndex];
		bool bChannelUsed[4] = {0};
		FVector4f InvUniformPenumbraSize(0, 0, 0, 0);

		for (int32 ChannelIndex = 0; ChannelIndex < 4; ChannelIndex++)
		{
			for (const auto& ShadowMapPair : Allocation.ShadowMapData)
			{
				ULightComponent* CurrentLight = ShadowMapPair.Key;
				UMapBuildDataRegistry* Registry = LightingContext->GetOrCreateRegistryForActor(CurrentLight->GetOwner());
				const FLightComponentMapBuildData* LightBuildData = Registry->GetLightBuildData(CurrentLight->LightGuid);

				// Should have been setup by ReassignStationaryLightChannels
				check(LightBuildData);

				if (LightBuildData->ShadowMapChannel == ChannelIndex)
				{
					MaxChannelsUsed = FMath::Max(MaxChannelsUsed, ChannelIndex + 1);
					bChannelUsed[ChannelIndex] = true;
					const TArray<FQuantizedSignedDistanceFieldShadowSample>& SourceSamples = ShadowMapPair.Value;

					// Warning - storing one penumbra size for the whole shadowmap even though multiple lights can share a channel
					InvUniformPenumbraSize[ChannelIndex] = 1.0f / ShadowMapPair.Key->GetUniformPenumbraSize();

					// Copy the raw data for this light-map into the raw texture data array.
					for (int32 Y = Allocation.MappedRect.Min.Y; Y < Allocation.MappedRect.Max.Y; ++Y)
					{
						for (int32 X = Allocation.MappedRect.Min.X; X < Allocation.MappedRect.Max.X; ++X)
						{
							int32 DestY = Y - Allocation.MappedRect.Min.Y + Allocation.OffsetY;
							int32 DestX = X - Allocation.MappedRect.Min.X + Allocation.OffsetX;

							FFourDistanceFieldSamples& DestSample = (*TopMipData)[DestY * TextureSizeX + DestX];
							const FQuantizedSignedDistanceFieldShadowSample& SourceSample = SourceSamples[Y * Allocation.TotalSizeX + X];

							if ( SourceSample.Coverage > 0 )
							{
								// Note: multiple lights can write to different parts of the destination due to channel assignment
								DestSample.Samples[ChannelIndex] = SourceSample;
							}
#if WITH_EDITOR
							if ( SourceSample.Coverage > 0 )
							{
								GNumShadowmapMappedTexels++;
							}
							else
							{
								GNumShadowmapUnmappedTexels++;
							}
#endif
						}
					}
				}
			}
		}

		// Link the shadow-map to the texture.
		Allocation.ShadowMap->Texture = Texture;

		// Free the shadow-map's raw data.
		for (auto& ShadowMapPair : Allocation.ShadowMapData)
		{
			ShadowMapPair.Value.Empty();
		}
		
		int32 PaddedSizeX = Allocation.TotalSizeX;
		int32 PaddedSizeY = Allocation.TotalSizeY;
		int32 BaseX = Allocation.OffsetX - Allocation.MappedRect.Min.X;
		int32 BaseY = Allocation.OffsetY - Allocation.MappedRect.Min.Y;

#if WITH_EDITOR
		if (GLightmassDebugOptions.bPadMappings && (Allocation.PaddingType == LMPT_NormalPadding))
		{
			if ((PaddedSizeX - 2 > 0) && ((PaddedSizeY - 2) > 0))
			{
				PaddedSizeX -= 2;
				PaddedSizeY -= 2;
				BaseX += 1;
				BaseY += 1;
			}
		}
#endif	
		// Calculate the coordinate scale/biases for each shadow-map stored in the texture.
		Allocation.ShadowMap->CoordinateScale = FVector2D(
			(float)PaddedSizeX / (float)PendingTexture.GetSizeX(),
			(float)PaddedSizeY / (float)PendingTexture.GetSizeY()
			);
		Allocation.ShadowMap->CoordinateBias = FVector2D(
			(float)BaseX / (float)PendingTexture.GetSizeX(),
			(float)BaseY / (float)PendingTexture.GetSizeY()
			);

		for (int32 ChannelIndex = 0; ChannelIndex < 4; ChannelIndex++)
		{
			Allocation.ShadowMap->bChannelValid[ChannelIndex] = bChannelUsed[ChannelIndex];
		}

		Allocation.ShadowMap->InvUniformPenumbraSize = InvUniformPenumbraSize;
	}

	const uint32 NumMips = FMath::Max(FMath::CeilLogTwo(TextureSizeX),FMath::CeilLogTwo(TextureSizeY)) + 1;

	for (uint32 MipIndex = 1;MipIndex < NumMips;MipIndex++)
	{
		const uint32 SourceMipSizeX = FMath::Max(1, TextureSizeX >> (MipIndex - 1));
		const uint32 SourceMipSizeY = FMath::Max(1, TextureSizeY >> (MipIndex - 1));
		const uint32 DestMipSizeX = FMath::Max(1, TextureSizeX >> MipIndex);
		const uint32 DestMipSizeY = FMath::Max(1, TextureSizeY >> MipIndex);

		// Downsample the previous mip-level, taking into account which texels are mapped.
		TArray<FFourDistanceFieldSamples>* NextMipData = new(MipData) TArray<FFourDistanceFieldSamples>();
		NextMipData->Empty(DestMipSizeX * DestMipSizeY);
		NextMipData->AddZeroed(DestMipSizeX * DestMipSizeY);
		const uint32 MipFactorX = SourceMipSizeX / DestMipSizeX;
		const uint32 MipFactorY = SourceMipSizeY / DestMipSizeY;

		for (uint32 Y = 0;Y < DestMipSizeY;Y++)
		{
			for (uint32 X = 0;X < DestMipSizeX;X++)
			{
				float AccumulatedFilterableComponents[4][FQuantizedSignedDistanceFieldShadowSample::NumFilterableComponents];

				for (int32 ChannelIndex = 0; ChannelIndex < 4; ChannelIndex++)
				{
					for (int32 i = 0; i < FQuantizedSignedDistanceFieldShadowSample::NumFilterableComponents; i++)
					{
						AccumulatedFilterableComponents[ChannelIndex][i] = 0;
					}
				}
				uint32 Coverage[4] = {0};

				for (uint32 SourceY = Y * MipFactorY;SourceY < (Y + 1) * MipFactorY;SourceY++)
				{
					for (uint32 SourceX = X * MipFactorX;SourceX < (X + 1) * MipFactorX;SourceX++)
					{
						for (int32 ChannelIndex = 0; ChannelIndex < 4; ChannelIndex++)
						{
							const FFourDistanceFieldSamples& FourSourceSamples = MipData[MipIndex - 1][SourceY * SourceMipSizeX + SourceX];
							const FQuantizedSignedDistanceFieldShadowSample& SourceSample = FourSourceSamples.Samples[ChannelIndex];

							if (SourceSample.Coverage)
							{
								for (int32 i = 0; i < FQuantizedSignedDistanceFieldShadowSample::NumFilterableComponents; i++)
								{
									AccumulatedFilterableComponents[ChannelIndex][i] += SourceSample.GetFilterableComponent(i) * SourceSample.Coverage;
								}

								Coverage[ChannelIndex] += SourceSample.Coverage;
							}
						}
					}
				}

				FFourDistanceFieldSamples& FourDestSamples = (*NextMipData)[Y * DestMipSizeX + X];

				for (int32 ChannelIndex = 0; ChannelIndex < 4; ChannelIndex++)
				{
					FQuantizedSignedDistanceFieldShadowSample& DestSample = FourDestSamples.Samples[ChannelIndex];

					if (Coverage[ChannelIndex])
					{
						for (int32 i = 0; i < FQuantizedSignedDistanceFieldShadowSample::NumFilterableComponents; i++)
						{
							DestSample.SetFilterableComponent(AccumulatedFilterableComponents[ChannelIndex][i] / (float)Coverage[ChannelIndex], i);
						}

						DestSample.Coverage = (uint8)(Coverage[ChannelIndex] / (MipFactorX * MipFactorY));
					}
					else
					{
						for (int32 i = 0; i < FQuantizedSignedDistanceFieldShadowSample::NumFilterableComponents; i++)
						{
							AccumulatedFilterableComponents[ChannelIndex][i] = 0;
						}
						DestSample.Coverage = 0;
					}
				}
			}
		}
	}

	const FIntPoint Neighbors[] = 
	{
		// Check immediate neighbors first
		FIntPoint(1,0),
		FIntPoint(0,1),
		FIntPoint(-1,0),
		FIntPoint(0,-1),
		// Check diagonal neighbors if no immediate neighbors are found
		FIntPoint(1,1),
		FIntPoint(-1,1),
		FIntPoint(-1,-1),
		FIntPoint(1,-1)
	};

	// Extrapolate texels which are mapped onto adjacent texels which are not mapped to avoid artifacts when using texture filtering.
	for (int32 MipIndex = 0;MipIndex < MipData.Num();MipIndex++)
	{
		uint32 MipSizeX = FMath::Max(1,TextureSizeX >> MipIndex);
		uint32 MipSizeY = FMath::Max(1,TextureSizeY >> MipIndex);

		for (uint32 DestY = 0;DestY < MipSizeY;DestY++)
		{
			for (uint32 DestX = 0;DestX < MipSizeX;DestX++)
			{
				FFourDistanceFieldSamples& FourDestSamples = MipData[MipIndex][DestY * MipSizeX + DestX];

				for (int32 ChannelIndex = 0; ChannelIndex < 4; ChannelIndex++)
				{
					FQuantizedSignedDistanceFieldShadowSample& DestSample = FourDestSamples.Samples[ChannelIndex];

					if (DestSample.Coverage == 0)
					{
						float ExtrapolatedFilterableComponents[FQuantizedSignedDistanceFieldShadowSample::NumFilterableComponents];

						for (int32 i = 0; i < FQuantizedSignedDistanceFieldShadowSample::NumFilterableComponents; i++)
						{
							ExtrapolatedFilterableComponents[i] = 0;
						}

						for (int32 NeighborIndex = 0; NeighborIndex < UE_ARRAY_COUNT(Neighbors); NeighborIndex++)
						{
							if (static_cast<int32>(DestY) + Neighbors[NeighborIndex].Y >= 0 
								&& DestY + Neighbors[NeighborIndex].Y < MipSizeY
								&& static_cast<int32>(DestX) + Neighbors[NeighborIndex].X >= 0
								&& DestX + Neighbors[NeighborIndex].X < MipSizeX)
							{
								const FFourDistanceFieldSamples& FourNeighborSamples = MipData[MipIndex][(DestY + Neighbors[NeighborIndex].Y) * MipSizeX + DestX + Neighbors[NeighborIndex].X];
								const FQuantizedSignedDistanceFieldShadowSample& NeighborSample = FourNeighborSamples.Samples[ChannelIndex];

								if (NeighborSample.Coverage > 0)
								{
									if (static_cast<int32>(DestY) + Neighbors[NeighborIndex].Y * 2 >= 0
										&& DestY + Neighbors[NeighborIndex].Y * 2 < MipSizeY
										&& static_cast<int32>(DestX) + Neighbors[NeighborIndex].X * 2 >= 0
										&& DestX + Neighbors[NeighborIndex].X * 2 < MipSizeX)
									{
										// Lookup the second neighbor in the first neighbor's direction
										//@todo - check the second neighbor's coverage?
										const FFourDistanceFieldSamples& SecondFourNeighborSamples = MipData[MipIndex][(DestY + Neighbors[NeighborIndex].Y * 2) * MipSizeX + DestX + Neighbors[NeighborIndex].X * 2];
										const FQuantizedSignedDistanceFieldShadowSample& SecondNeighborSample = FourNeighborSamples.Samples[ChannelIndex];

										for (int32 i = 0; i < FQuantizedSignedDistanceFieldShadowSample::NumFilterableComponents; i++)
										{
											// Extrapolate while maintaining the first derivative, which is especially important for signed distance fields
											ExtrapolatedFilterableComponents[i] = NeighborSample.GetFilterableComponent(i) * 2.0f - SecondNeighborSample.GetFilterableComponent(i);
										}
									}
									else
									{
										// Couldn't find a second neighbor to use for extrapolating, just copy the neighbor's values
										for (int32 i = 0; i < FQuantizedSignedDistanceFieldShadowSample::NumFilterableComponents; i++)
										{
											ExtrapolatedFilterableComponents[i] = NeighborSample.GetFilterableComponent(i);
										}
									}
									break;
								}
							}
						}
						for (int32 i = 0; i < FQuantizedSignedDistanceFieldShadowSample::NumFilterableComponents; i++)
						{
							DestSample.SetFilterableComponent(ExtrapolatedFilterableComponents[i], i);
						}
					}
				}
			}
		}
	}

	for (auto& Allocation : PendingTexture.Allocations)
	{
		Allocation->PostEncode();
	}

	return MaxChannelsUsed;
}

#endif

FArchive& operator<<(FArchive& Ar,FShadowMap*& R)
{
	uint32 ShadowMapType = FShadowMap::SMT_None;

	if (Ar.IsSaving())
	{
		if (R != nullptr)
		{
			if (R->GetShadowMap2D())
			{
				ShadowMapType = FShadowMap::SMT_2D;
			}
		}
	}

	Ar << ShadowMapType;

	if (Ar.IsLoading())
	{
		// explicitly don't call "delete R;",
		// we expect the calling code to handle that
		switch (ShadowMapType)
		{
		case FShadowMap::SMT_None:
			R = nullptr;
			break;
		case FShadowMap::SMT_2D:
			R = new FShadowMap2D();
			break;
		default:
			check(0);
		}
	}

	if (R != nullptr)
	{
		R->Serialize(Ar);

		if (Ar.IsLoading())
		{
			// Dump old Shadowmaps
			if (Ar.UEVer() < VER_UE4_COMBINED_LIGHTMAP_TEXTURES)
			{
				delete R; // safe because if we're loading we new'd this above
				R = nullptr;
			}
		}
	}

	return Ar;
}
