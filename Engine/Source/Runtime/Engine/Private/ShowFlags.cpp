// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	ShowFlags.cpp: Show Flag Definitions.
=============================================================================*/

#include "ShowFlags.h"
#include "RenderUtils.h"
#include "Engine/EngineBaseTypes.h"
#include "HAL/IConsoleManager.h"
#include "Misc/ScopeRWLock.h"
#include "Misc/TransactionallySafeRWLock.h"
#include "SystemSettings.h"

static bool IsValidNameChar(TCHAR c)
{
	return (c >= (TCHAR)'a' && c <= (TCHAR)'z')
		|| (c >= (TCHAR)'A' && c <= (TCHAR)'Z')
		|| (c >= (TCHAR)'0' && c <= (TCHAR)'9')
		|| (c == (TCHAR)'_'); 
}

static void SkipWhiteSpace(const TCHAR*& p)
{
	for(;;)
	{
		if(IsValidNameChar(*p) || *p == (TCHAR)',' || *p == (TCHAR)'=')
		{
			return;
		}

		++p;
	}
}

// ----------------------------------------------------------------------------

FString FEngineShowFlags::ToString() const
{
	struct FIterSink
	{
		FIterSink(const FEngineShowFlags InEngineShowFlags) : EngineShowFlags(InEngineShowFlags)
		{
		}

		bool HandleShowFlag(uint32 InIndex, const FString& InName)
		{
			EShowFlagGroup Group = FEngineShowFlags::FindShowFlagGroup(*InName);
			if (Group != SFG_Transient)
			{
				if (!ret.IsEmpty())
				{
					ret += (TCHAR)',';
				}

				AddNameByIndex(InIndex, ret);

				ret += (TCHAR)'=';
				ret += EngineShowFlags.GetSingleFlag(InIndex) ? (TCHAR)'1' : (TCHAR)'0';
			}
			return true;
		}

		bool OnEngineShowFlag(uint32 InIndex, const FString& InName)
		{
			return HandleShowFlag(InIndex, InName);
		}

		bool OnCustomShowFlag(uint32 InIndex, const FString& InName)
		{
			return HandleShowFlag(InIndex, InName);
		}

		FString ret;
		const FEngineShowFlags EngineShowFlags;
	};

	FIterSink Sink(*this);

	IterateAllFlags(Sink);

	return Sink.ret;
}

bool FEngineShowFlags::SetFromString(const TCHAR* In)
{
	bool bError = false;

	const TCHAR* p = In;

	SkipWhiteSpace(p);

	while(*p)
	{
		FString Name;

		// jump over name
		while(IsValidNameChar(*p))
		{
			Name += *p++;
		}

		int32 Index = FindIndexByName(*Name);

		// true:set false:clear
		bool bSet = true;

		if(*p == (TCHAR)'=')
		{
			++p;
			if(*p == (TCHAR)'0')
			{
				bSet = false;
			}
			++p;
		}

		if(Index == INDEX_NONE)
		{
			// unknown name but we try to parse further
			bError = true;
		}
		else
		{
			SetSingleFlag(Index, bSet);
		}

		if(*p == (TCHAR)',')
		{
			++p;
		}
		else
		{
			// parse error;
			return false;
		}
	}

	return !bError;
}

bool FEngineShowFlags::GetSingleFlag(uint32 Index) const
{
	switch( Index )
	{
	#define SHOWFLAG_ALWAYS_ACCESSIBLE(a,...) case SF_##a: return a != 0;
	#include "ShowFlagsValues.inl" // IWYU pragma: keep
	default:
		if (Index >= SF_FirstCustom)
		{
			ECustomShowFlag CustomShowFlag = ECustomShowFlag(Index - SF_FirstCustom);
			if (CustomShowFlags.IsValidIndex((uint32)CustomShowFlag))
			{
				return CustomShowFlags[(uint32)CustomShowFlag];
			}
			else if (IsRegisteredCustomShowFlag(CustomShowFlag))
			{
				return false;
			}
		}
		{
			checkNoEntry();
			return false;
		}
	}
}

bool FEngineShowFlags::IsForceFlagSet(uint32 Index)
{
	const uint8* Force0Ptr = (const uint8*)&GSystemSettings.GetForce0Mask();
	const uint8* Force1Ptr = (const uint8*)&GSystemSettings.GetForce1Mask();

	bool Force0Set = *(Force0Ptr + Index / CHAR_BIT) & (1 << Index % CHAR_BIT);
	bool Force1Set = *(Force1Ptr + Index / CHAR_BIT) & (1 << Index % CHAR_BIT);

	return !Force0Set && !Force1Set;
}

void FEngineShowFlags::SetSingleFlag(uint32 Index, bool bSet)
{
	switch( Index )
	{
	#define SHOWFLAG_ALWAYS_ACCESSIBLE(a,...) case SF_##a: a = bSet?1:0; break;
	#if UE_BUILD_OPTIMIZED_SHOWFLAGS 
		#define SHOWFLAG_FIXED_IN_SHIPPING(v,a,...) case SF_##a: break;
	#endif
	#include "ShowFlagsValues.inl" // IWYU pragma: keep
	default:
		UpdateNewCustomShowFlags();
		if (Index >= SF_FirstCustom && (Index - SF_FirstCustom) < (uint32)CustomShowFlags.Num())
		{
			CustomShowFlags[Index - SF_FirstCustom] = bSet;
			return;
		}
		{
			checkNoEntry();
		}
	}
}

int32 FEngineShowFlags::FindIndexByName(const TCHAR* Name, const TCHAR* CommaSeparatedNames)
{
	if (!Name)
	{
		// invalid input
		return INDEX_NONE;
	}

	if (CommaSeparatedNames == nullptr)
	{
		// search through all defined show flags.
		FString Search = Name;

		#define SHOWFLAG_ALWAYS_ACCESSIBLE(a,...) if(Search == UE_STRINGIZE(a)) { return (int32)SF_##a; }

		#include "ShowFlagsValues.inl" // IWYU pragma: keep

		ECustomShowFlag CustomFlagIndex = FindCustomShowFlagByName(Search);
		if (CustomFlagIndex != ECustomShowFlag::None)
		{
			return SF_FirstCustom + (uint32)CustomFlagIndex;
		}
		return INDEX_NONE;
	}
	else
	{
		// iterate through CommaSeparatedNames and test 'Name' equals one of them.
		struct FIterSink
		{
			FIterSink(const TCHAR* InName)
			{
				SearchName = InName;
				Ret = INDEX_NONE;
			}

			bool OnEngineShowFlag(uint32 InIndex, const FString& InName)
			{
				if (InName == SearchName)
				{
					Ret = InIndex;
					return false;
				}
				return true;
			}
			const TCHAR* SearchName;
			uint32 Ret;
		};
		FIterSink Sink(Name);
		IterateAllFlags(Sink, CommaSeparatedNames);
		return Sink.Ret;
	}
}

// Codegen optimization degenerates for very long functions like FindNameByIndex.
// We don't need this code to be particularly fast anyway.
BEGIN_FUNCTION_BUILD_OPTIMIZATION

FString FEngineShowFlags::FindNameByIndex(uint32 InIndex)
{
	FString Name;

	#define SHOWFLAG_ALWAYS_ACCESSIBLE(a,...) case SF_##a: Name = UE_STRINGIZE(a); break;

	switch (InIndex)
	{
		#include "ShowFlagsValues.inl" // IWYU pragma: keep
	default:
		return GetCustomShowFlagName(ECustomShowFlag(InIndex - SF_FirstCustom));
	}

	return Name;
}

END_FUNCTION_BUILD_OPTIMIZATION

void FEngineShowFlags::AddNameByIndex(uint32 InIndex, FString& Out)
{
	#define SHOWFLAG_ALWAYS_ACCESSIBLE(a,...) case SF_##a: Out += UE_STRINGIZE(a); break;
	switch (InIndex)
	{
		#include "ShowFlagsValues.inl" // IWYU pragma: keep
		default:
			FString Name = GetCustomShowFlagName(ECustomShowFlag(InIndex - SF_FirstCustom));
			if (Name.IsEmpty() == false)
			{
				Out += Name;
			}
			break;
	}
}

void ApplyViewMode(EViewModeIndex ViewModeIndex, bool bPerspective, FEngineShowFlags& EngineShowFlags)
{
	bool bPostProcessing = true;

	switch(ViewModeIndex)
	{
		case VMI_BrushWireframe:
			bPostProcessing = false;
			break;
		case VMI_Wireframe:
			bPostProcessing = false;
			break;
		case VMI_Unlit:
			bPostProcessing = false;
			break;
		default:
		case VMI_Lit: 
			bPostProcessing = true;
			break;
		case VMI_Lit_DetailLighting:
			bPostProcessing = true;
			break;
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		case VMI_Lit_Wireframe:
			bPostProcessing = true;
			break;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
		case VMI_LightingOnly:
			bPostProcessing = true;
			break;
		case VMI_LightComplexity:
			bPostProcessing = false;
			break;
		case VMI_ShaderComplexity:
		case VMI_QuadOverdraw:
		case VMI_ShaderComplexityWithQuadOverdraw:
		case VMI_PrimitiveDistanceAccuracy:
		case VMI_MeshUVDensityAccuracy:
		case VMI_MaterialTextureScaleAccuracy:
		case VMI_RequiredTextureResolution:
		case VMI_LODColoration:
		case VMI_HLODColoration:
		case VMI_VisualizeActorColoration:
		case VMI_StreamingTextureDeficit:
		case VMI_StreamingTextureResidency:
		case VMI_StreamingMeshLODDeficit:
		case VMI_StreamingMeshLODResidency:
			bPostProcessing = false;
			break;
		case VMI_StationaryLightOverlap:
			bPostProcessing = false;
			break;
		case VMI_LightmapDensity:
			bPostProcessing = false;
			break;
		case VMI_LitLightmapDensity:
			bPostProcessing = false;
			break;
		case VMI_VisualizeBuffer:
			bPostProcessing = true;
			break;
		case VMI_VisualizeNanite:
			bPostProcessing = true;
			break;
		case VMI_VisualizeLumen:
			bPostProcessing = true;
			break;
		case VMI_VisualizeMegaLights:
			bPostProcessing = true;
			break;
		case VMI_VisualizeSubstrate:
			bPostProcessing = true;
			break;
		case VMI_VisualizeGroom:
			bPostProcessing = true;
			break;
		case VMI_VisualizeVirtualShadowMap:
			bPostProcessing = true;
			break;
		case VMI_VisualizeGPUSkinCache:
			bPostProcessing = false;
			break;
		case VMI_LWCComplexity:
			bPostProcessing = false;
			break;
		case VMI_ReflectionOverride:
			bPostProcessing = true;
			break;
		case VMI_CollisionPawn:
		case VMI_CollisionVisibility:
			bPostProcessing = false;
			break;
		case VMI_RayTracingDebug:
			bPostProcessing = true;
			break;
		case VMI_PathTracing:
			bPostProcessing = true;
			break;
		case VMI_Zebra:
		case VMI_Clay:
		case VMI_FrontBackFace:
		case VMI_RandomColor:
			bPostProcessing = true;
			break;
	}

	// set the EngineShowFlags:

	// Assigning the new state like this ensures we always set the same variables (they depend on the view mode)
	// This is affecting the state of show flags - if the state can be changed by the user as well it should better be done in EngineShowFlagOverride

	EngineShowFlags.SetOverrideDiffuseAndSpecular(ViewModeIndex == VMI_Lit_DetailLighting);
	EngineShowFlags.SetLightingOnlyOverride(ViewModeIndex == VMI_LightingOnly);
	EngineShowFlags.SetReflectionOverride(ViewModeIndex == VMI_ReflectionOverride);
	EngineShowFlags.SetVisualizeBuffer(ViewModeIndex == VMI_VisualizeBuffer);
	EngineShowFlags.SetVisualizeNanite(ViewModeIndex == VMI_VisualizeNanite);
	EngineShowFlags.SetVisualizeLumen(ViewModeIndex == VMI_VisualizeLumen);
	EngineShowFlags.SetVisualizeMegaLights(ViewModeIndex == VMI_VisualizeMegaLights);
	EngineShowFlags.SetVisualizeSubstrate(ViewModeIndex == VMI_VisualizeSubstrate);
	EngineShowFlags.SetVisualizeGroom(ViewModeIndex == VMI_VisualizeGroom);
	EngineShowFlags.SetVisualizeVirtualShadowMap(ViewModeIndex == VMI_VisualizeVirtualShadowMap);
	EngineShowFlags.SetVisualizeVirtualTexture(ViewModeIndex == VMI_VisualizeVirtualTexture);
	EngineShowFlags.SetVisualizeLightCulling(ViewModeIndex == VMI_LightComplexity);
	EngineShowFlags.SetShaderComplexity(ViewModeIndex == VMI_ShaderComplexity || ViewModeIndex == VMI_QuadOverdraw || ViewModeIndex == VMI_ShaderComplexityWithQuadOverdraw);
	EngineShowFlags.SetQuadOverdraw(ViewModeIndex == VMI_QuadOverdraw);
	EngineShowFlags.SetShaderComplexityWithQuadOverdraw(ViewModeIndex == VMI_ShaderComplexityWithQuadOverdraw);
	EngineShowFlags.SetPrimitiveDistanceAccuracy(ViewModeIndex == VMI_PrimitiveDistanceAccuracy);
	EngineShowFlags.SetMeshUVDensityAccuracy(ViewModeIndex == VMI_MeshUVDensityAccuracy);
	EngineShowFlags.SetMaterialTextureScaleAccuracy(ViewModeIndex == VMI_MaterialTextureScaleAccuracy);
	EngineShowFlags.SetRequiredTextureResolution(ViewModeIndex == VMI_RequiredTextureResolution);
	EngineShowFlags.SetStationaryLightOverlap(ViewModeIndex == VMI_StationaryLightOverlap);
	EngineShowFlags.SetLightMapDensity(ViewModeIndex == VMI_LightmapDensity || ViewModeIndex == VMI_LitLightmapDensity);
	EngineShowFlags.SetPostProcessing(bPostProcessing);
	EngineShowFlags.SetBSPTriangles(ViewModeIndex != VMI_BrushWireframe && ViewModeIndex != VMI_LitLightmapDensity);
	EngineShowFlags.SetBrushes(ViewModeIndex == VMI_BrushWireframe);
	EngineShowFlags.SetWireframe(ViewModeIndex == VMI_Wireframe || ViewModeIndex == VMI_BrushWireframe);
	EngineShowFlags.SetCollisionPawn(ViewModeIndex == VMI_CollisionPawn);
	EngineShowFlags.SetCollisionVisibility(ViewModeIndex == VMI_CollisionVisibility);
	EngineShowFlags.SetLODColoration(ViewModeIndex == VMI_LODColoration);
	EngineShowFlags.SetHLODColoration(ViewModeIndex == VMI_HLODColoration);
	EngineShowFlags.SetStreamingTextureDeficit(ViewModeIndex == VMI_StreamingTextureDeficit);
	EngineShowFlags.SetStreamingTextureResidency(ViewModeIndex == VMI_StreamingTextureResidency);
	EngineShowFlags.SetStreamingMeshLODDeficit(ViewModeIndex == VMI_StreamingMeshLODDeficit);
	EngineShowFlags.SetStreamingMeshLODResidency(ViewModeIndex == VMI_StreamingMeshLODResidency);
	EngineShowFlags.SetRayTracingDebug(ViewModeIndex == VMI_RayTracingDebug);
	EngineShowFlags.SetPathTracing(ViewModeIndex == VMI_PathTracing);
	EngineShowFlags.SetVisualizeGPUSkinCache(ViewModeIndex == VMI_VisualizeGPUSkinCache);
	EngineShowFlags.SetVisualizeLWCComplexity(ViewModeIndex == VMI_LWCComplexity);
	EngineShowFlags.SetVisualizeShadowCasters(ViewModeIndex == VMI_ShadowCasters);
	EngineShowFlags.SetActorColoration(ViewModeIndex == VMI_VisualizeActorColoration);
	EngineShowFlags.SetClay(ViewModeIndex == VMI_Clay);
	EngineShowFlags.SetZebra(ViewModeIndex == VMI_Zebra);
	EngineShowFlags.SetFrontBackFace(ViewModeIndex == VMI_FrontBackFace);
	EngineShowFlags.SetRandomColor(ViewModeIndex == VMI_RandomColor);
}

void EngineShowFlagOverride(EShowFlagInitMode ShowFlagInitMode, EViewModeIndex ViewModeIndex, FEngineShowFlags& EngineShowFlags, bool bCanDisableTonemapper)
{
	if (ShowFlagInitMode == ESFIM_Game)
	{
		// Editor only features
		EngineShowFlags.SetAudioRadius(false);
	}

	{
		// When taking a high resolution screenshot
		if (GIsHighResScreenshot)
		{
			static const auto ICVar = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.HighResScreenshotDelay"));
			if(ICVar->GetValueOnGameThread() < 4)
			{
				// Disabled as it requires multiple frames, AA can be done by downsampling, more control and better masking
				EngineShowFlags.SetTemporalAA(false);
			}

			// No editor gizmos / selection
			EngineShowFlags.SetModeWidgets(false);
			EngineShowFlags.SetSelection(false);
			EngineShowFlags.SetSelectionOutline(false);
		}
	}

	{
		static const auto ICVar = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.LightFunctionQuality"));
		if(ICVar->GetValueOnGameThread() <= 0)
		{
			EngineShowFlags.SetLightFunctions(false);
		}
	}

	{
		static const auto ICVar = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.EyeAdaptationQuality"));
		if(ICVar->GetValueOnGameThread() <= 0)
		{
			EngineShowFlags.SetEyeAdaptation(false);
		}
	}

	{
		static const auto ICVar = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.ShadowQuality"));
		if(ICVar->GetValueOnGameThread() <= 0)
		{
			EngineShowFlags.SetDynamicShadows(false);
		}
	}

	{
		static const auto ICVar = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.SkyLightingQuality"));
		if (ICVar->GetValueOnGameThread() <= 0)
		{
			EngineShowFlags.SetSkyLighting(false);
		}
	}

	{
		static const auto ICVar = IConsoleManager::Get().FindTConsoleVariableDataBool(TEXT("r.AlphaInvertPass"));
		if (ICVar->GetValueOnGameThread())
		{
			EngineShowFlags.SetAlphaInvert(true); 
		}
	}

	if (!IsRayTracingEnabled())
	{
		EngineShowFlags.SetPathTracing(false);
		EngineShowFlags.SetRayTracingDebug(false);
	}

	// Some view modes want some features off or on (no state)
	{
		if (ViewModeIndex == VMI_BrushWireframe ||
			ViewModeIndex == VMI_Wireframe)
		{
			EngineShowFlags.SetWireframe(true);
		}
		else
		{
			EngineShowFlags.SetWireframe(false);
		}

		if( ViewModeIndex == VMI_BrushWireframe ||
			ViewModeIndex == VMI_Wireframe ||
			ViewModeIndex == VMI_Unlit ||
			ViewModeIndex == VMI_LightmapDensity ||
			ViewModeIndex == VMI_LitLightmapDensity)
		{
			EngineShowFlags.SetLightFunctions(false);
		}

		if( ViewModeIndex == VMI_BrushWireframe ||
			ViewModeIndex == VMI_Wireframe ||
			ViewModeIndex == VMI_Unlit ||
			ViewModeIndex == VMI_ShaderComplexity ||
			ViewModeIndex == VMI_QuadOverdraw ||
			ViewModeIndex == VMI_ShaderComplexityWithQuadOverdraw ||
			ViewModeIndex == VMI_PrimitiveDistanceAccuracy ||
			ViewModeIndex == VMI_MeshUVDensityAccuracy ||
			ViewModeIndex == VMI_MaterialTextureScaleAccuracy ||
			ViewModeIndex == VMI_RequiredTextureResolution ||
			ViewModeIndex == VMI_LightmapDensity ||
			ViewModeIndex == VMI_LitLightmapDensity)
		{
			EngineShowFlags.SetDynamicShadows(false);
		}

		if( ViewModeIndex == VMI_BrushWireframe)
		{
			EngineShowFlags.SetBrushes(true);
		}

		if( ViewModeIndex == VMI_Unlit ||
			ViewModeIndex == VMI_Wireframe ||
			ViewModeIndex == VMI_BrushWireframe ||
			ViewModeIndex == VMI_CollisionPawn ||
			ViewModeIndex == VMI_CollisionVisibility ||
			ViewModeIndex == VMI_StationaryLightOverlap ||
			ViewModeIndex == VMI_ShaderComplexity ||
			ViewModeIndex == VMI_QuadOverdraw ||
			ViewModeIndex == VMI_ShaderComplexityWithQuadOverdraw ||
			ViewModeIndex == VMI_PrimitiveDistanceAccuracy ||
			ViewModeIndex == VMI_MeshUVDensityAccuracy ||
			ViewModeIndex == VMI_MaterialTextureScaleAccuracy ||
			ViewModeIndex == VMI_RequiredTextureResolution ||
			ViewModeIndex == VMI_LODColoration ||
			ViewModeIndex == VMI_HLODColoration ||
			ViewModeIndex == VMI_VisualizeActorColoration ||
			ViewModeIndex == VMI_VisualizeGPUSkinCache ||
			ViewModeIndex == VMI_LWCComplexity ||
			ViewModeIndex == VMI_LightmapDensity ||
			ViewModeIndex == VMI_ShadowCasters ||
			ViewModeIndex == VMI_StreamingTextureDeficit ||
			ViewModeIndex == VMI_StreamingTextureResidency ||
			ViewModeIndex == VMI_StreamingMeshLODDeficit ||
			ViewModeIndex == VMI_StreamingMeshLODResidency)
		{
			EngineShowFlags.SetLighting(false);
			EngineShowFlags.SetAtmosphere(false);
			EngineShowFlags.SetFog(false);
		}

		if (ViewModeIndex == VMI_Clay ||
			ViewModeIndex == VMI_Zebra ||
			ViewModeIndex == VMI_FrontBackFace ||
			ViewModeIndex == VMI_RandomColor)
		{
			EngineShowFlags.SetLighting(false);
			EngineShowFlags.SetCapsuleShadows(false);
			EngineShowFlags.SetContactShadows(false);
			EngineShowFlags.SetDynamicShadows(false);

			EngineShowFlags.SetAtmosphere(false);
			EngineShowFlags.SetFog(false);
		}

		if (ViewModeIndex == VMI_ShadowCasters)
		{
			EngineShowFlags.SetTemporalAA(false);
			EngineShowFlags.SetEditor(false);
		}

		if( ViewModeIndex == VMI_Lit ||
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
			ViewModeIndex == VMI_Lit_Wireframe ||
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
			ViewModeIndex == VMI_LightingOnly ||
			ViewModeIndex == VMI_LitLightmapDensity ||
			ViewModeIndex == VMI_VisualizeVirtualShadowMap )
		{
			EngineShowFlags.SetLighting(true);
		}

		if( ViewModeIndex == VMI_LightingOnly ||
			ViewModeIndex == VMI_BrushWireframe ||
			ViewModeIndex == VMI_StationaryLightOverlap)
		{
			EngineShowFlags.SetMaterials(false);
		}

		if( ViewModeIndex == VMI_LightComplexity )
		{
			EngineShowFlags.Translucency = 0;
			EngineShowFlags.SetFog(false);
			EngineShowFlags.SetAtmosphere(false);
		}

		if (ViewModeIndex == VMI_PrimitiveDistanceAccuracy ||
			ViewModeIndex == VMI_MeshUVDensityAccuracy ||
			ViewModeIndex == VMI_MaterialTextureScaleAccuracy || 
			ViewModeIndex == VMI_RequiredTextureResolution)
		{
			EngineShowFlags.SetDecals(false); // Decals require the use of FDebugPSInLean.
			EngineShowFlags.SetParticles(false); // FX are fully streamed.
			EngineShowFlags.SetFog(false);
		}

		if (ViewModeIndex == VMI_LODColoration ||
			ViewModeIndex == VMI_HLODColoration ||
			ViewModeIndex == VMI_VisualizeGPUSkinCache ||
			ViewModeIndex == VMI_StreamingTextureDeficit ||
			ViewModeIndex == VMI_StreamingTextureResidency ||
			ViewModeIndex == VMI_StreamingMeshLODDeficit ||
			ViewModeIndex == VMI_StreamingMeshLODResidency)
		{
			EngineShowFlags.SetDecals(false); // Decals require the use of FDebugPSInLean.
		}

		if (IsRayTracingEnabled() && ViewModeIndex == VMI_RayTracingDebug)
		{
			EngineShowFlags.SetVisualizeHDR(false);
			EngineShowFlags.SetVisualizeLocalExposure(false);
			EngineShowFlags.SetVisualizeMotionBlur(false);
			EngineShowFlags.SetDepthOfField(false);
			EngineShowFlags.SetPostProcessMaterial(false);

			if (bCanDisableTonemapper)
			{
				EngineShowFlags.SetTonemapper(false);
			}
		}

		if (ViewModeIndex == VMI_VisualizeNanite)
		{
			// TODO: NANITE_VIEW_MODES: Only disable these in fullscreen mode
			/*EngineShowFlags.SetVisualizeHDR(false);
			EngineShowFlags.SetVisualizeMotionBlur(false);
			EngineShowFlags.SetDepthOfField(false);
			EngineShowFlags.SetPostProcessMaterial(false);
			EngineShowFlags.SetTemporalAA(false);

			if (bCanDisableTonemapper)
			{
				EngineShowFlags.SetTonemapper(false);
			}*/
		}
		
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		if (ViewModeIndex == VMI_Lit_Wireframe)
		{
			EngineShowFlags.SetMeshEdges(true);
		}
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	// Disable AA in full screen GBuffer visualization or calibration material visualization
	if (bCanDisableTonemapper && EngineShowFlags.VisualizeBuffer)
	{
		EngineShowFlags.SetTonemapper(false);
	}
	
	if (EngineShowFlags.MeshEdges)
	{
		// TAA-induced jittering causes a mismatch between the sampled positions for wireframe and lit-view pixels, resulting in z-fighting and flickering.
		// Enabling jittering on the wireframe means also enabling TAA, which doesn't need it (MSAA) and causes ghosting artifacts.
		EngineShowFlags.SetTemporalAA(false);
	}


	if (EngineShowFlags.Bones)
	{
		// Disabling some post processing effects when debug rendering bones as they do not work properly together
		EngineShowFlags.SetTemporalAA(false);
		EngineShowFlags.SetMotionBlur(false);
		EngineShowFlags.SetBloom(false);
	}

	if (EngineShowFlags.VisualizeColorGrading)
	{
		EngineShowFlags.SetCompositeEditorPrimitives(false);
		EngineShowFlags.SetCompositeDebugPrimitives(false);
		EngineShowFlags.SetAudioRadius(false);
		EngineShowFlags.SetEditor(false);
		EngineShowFlags.SetGrid(false);
		EngineShowFlags.SetSplines(false);
		EngineShowFlags.SetSelection(false);
		EngineShowFlags.SetLightRadius(false);
	}

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	{
		static const auto ICVar = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.LimitRenderingFeatures"));
		if (ICVar)
		{
			 int32 Value = ICVar->GetValueOnGameThread();

#define DISABLE_ENGINE_SHOWFLAG(Name) if(Value-- >  0) EngineShowFlags.Set##Name(false);
			 DISABLE_ENGINE_SHOWFLAG(AntiAliasing)
			 DISABLE_ENGINE_SHOWFLAG(EyeAdaptation)
			 DISABLE_ENGINE_SHOWFLAG(SeparateTranslucency)
			 DISABLE_ENGINE_SHOWFLAG(DepthOfField)
			 DISABLE_ENGINE_SHOWFLAG(AmbientOcclusion)
			 DISABLE_ENGINE_SHOWFLAG(CameraImperfections)
			 DISABLE_ENGINE_SHOWFLAG(Decals)
			 DISABLE_ENGINE_SHOWFLAG(LensFlares)
			 DISABLE_ENGINE_SHOWFLAG(Bloom)
			 DISABLE_ENGINE_SHOWFLAG(ColorGrading)
			 DISABLE_ENGINE_SHOWFLAG(Tonemapper)
			 DISABLE_ENGINE_SHOWFLAG(Refraction)
			 DISABLE_ENGINE_SHOWFLAG(ReflectionEnvironment)
			 DISABLE_ENGINE_SHOWFLAG(AmbientCubemap)
			 DISABLE_ENGINE_SHOWFLAG(MotionBlur)
			 DISABLE_ENGINE_SHOWFLAG(DirectLighting)
			 DISABLE_ENGINE_SHOWFLAG(Lighting)
			 DISABLE_ENGINE_SHOWFLAG(Translucency)
			 DISABLE_ENGINE_SHOWFLAG(TextRender)
			 DISABLE_ENGINE_SHOWFLAG(Particles)
			 DISABLE_ENGINE_SHOWFLAG(SkeletalMeshes)
			 DISABLE_ENGINE_SHOWFLAG(StaticMeshes)
			 DISABLE_ENGINE_SHOWFLAG(NaniteMeshes)
			 DISABLE_ENGINE_SHOWFLAG(BSP)
			 DISABLE_ENGINE_SHOWFLAG(Paper2DSprites)
#undef DISABLE_ENGINE_SHOWFLAG
		}
	}
#endif

	// Force some show flags to be 0 or 1
	{
		const uint8* Force0Ptr = (const uint8*)&GSystemSettings.GetForce0Mask();
		const uint8* Force1Ptr = (const uint8*)&GSystemSettings.GetForce1Mask();
		uint8* Ptr = (uint8*)&EngineShowFlags;

		for (uint32 Iter = 0; Iter < sizeof(FEngineShowFlags); ++Iter)
		{
			uint8 Value = *Ptr;

			Value &= ~(*Force0Ptr);
			Value |= *Force1Ptr;
			*Ptr++ = Value;
			++Force0Ptr;
			++Force1Ptr;
		}
	}

	EngineShowFlags.EngineOverrideCustomShowFlagsFromCVars();
}

void EngineShowFlagOrthographicOverride(bool bIsPerspective, FEngineShowFlags& EngineShowFlags)
{
	if (!bIsPerspective)
	{
		/**
		 * Orthographic feature support has been improved so all features are now enabled by default.
		 * If you wish to disable a specific feature, add the showflag disable call here.
		 * e.g.	EngineShowFlags.SetMotionBlur(false);
		 */
	}
}

EViewModeIndex FindViewMode(const FEngineShowFlags& EngineShowFlags)
{
	if (EngineShowFlags.VisualizeBuffer)
	{
		return VMI_VisualizeBuffer;
	}
	else if (EngineShowFlags.VisualizeNanite)
	{
		return VMI_VisualizeNanite;
	}
	else if (EngineShowFlags.VisualizeLumen)
	{
		return VMI_VisualizeLumen;
	}
	else if (EngineShowFlags.VisualizeMegaLights)
	{
		return VMI_VisualizeMegaLights;
	}
	else if (EngineShowFlags.VisualizeSubstrate)
	{
		return VMI_VisualizeSubstrate;
	}
	else if (EngineShowFlags.VisualizeGroom)
	{
		return VMI_VisualizeGroom;
	}
	else if (EngineShowFlags.VisualizeVirtualShadowMap)
	{
		return VMI_VisualizeVirtualShadowMap;
	}
	else if (EngineShowFlags.StationaryLightOverlap)
	{
		return VMI_StationaryLightOverlap;
	}
	// Test QuadComplexity before ShaderComplexity because QuadComplexity also use ShaderComplexity
	else if (EngineShowFlags.QuadOverdraw)
	{
		return VMI_QuadOverdraw;
	}
	else if (EngineShowFlags.ShaderComplexityWithQuadOverdraw)
	{
		return VMI_ShaderComplexityWithQuadOverdraw;
	}
	else if (EngineShowFlags.PrimitiveDistanceAccuracy)
	{
		return VMI_PrimitiveDistanceAccuracy;
	}
	else if (EngineShowFlags.MeshUVDensityAccuracy)
	{
		return VMI_MeshUVDensityAccuracy;
	}
	else if (EngineShowFlags.MaterialTextureScaleAccuracy)
	{
		return VMI_MaterialTextureScaleAccuracy;
	}
	else if (EngineShowFlags.RequiredTextureResolution)
	{
		return VMI_RequiredTextureResolution;
	}
	else if (EngineShowFlags.VisualizeVirtualTexture)
	{
		return VMI_VisualizeVirtualTexture;
	}
	else if (EngineShowFlags.ShaderComplexity)
	{
		return VMI_ShaderComplexity;
	}
	else if (EngineShowFlags.VisualizeLightCulling)
	{
		return VMI_LightComplexity;
	}
	else if (EngineShowFlags.LightMapDensity)
	{
		if (EngineShowFlags.Lighting)
		{
			return VMI_LitLightmapDensity;
		}
		else
		{
			return VMI_LightmapDensity;
		}
	}
	else if (EngineShowFlags.OverrideDiffuseAndSpecular)
	{
		return VMI_Lit_DetailLighting;
	}
	else if (EngineShowFlags.LightingOnlyOverride)
	{
		return VMI_LightingOnly;
	}
	else if (EngineShowFlags.ReflectionOverride)
	{
		return VMI_ReflectionOverride;
	}
	else if (EngineShowFlags.Wireframe)
	{
		if (EngineShowFlags.Brushes)
		{
			return VMI_BrushWireframe;
		}
		else
		{
			return VMI_Wireframe;
		}
	}
	else if (!EngineShowFlags.Materials && EngineShowFlags.Lighting)
	{
		return VMI_LightingOnly;
	}
	else if (EngineShowFlags.CollisionPawn)
	{
		return VMI_CollisionPawn;
	}
	else if (EngineShowFlags.CollisionVisibility)
	{
		return VMI_CollisionVisibility;
	}
	else if (EngineShowFlags.LODColoration)
	{
		return VMI_LODColoration;
	}
	else if (EngineShowFlags.HLODColoration)
	{
		return VMI_HLODColoration;
	}
	else if (EngineShowFlags.StreamingTextureDeficit)
	{
		return VMI_StreamingTextureDeficit;
	}
	else if (EngineShowFlags.StreamingTextureResidency)
	{
		return VMI_StreamingTextureResidency;
	}
	else if (EngineShowFlags.StreamingMeshLODDeficit)
	{
		return VMI_StreamingMeshLODDeficit;
	}
	else if (EngineShowFlags.StreamingMeshLODResidency)
	{
		return VMI_StreamingMeshLODResidency;
	}
	else if (EngineShowFlags.PathTracing)
	{
		return VMI_PathTracing;
	}
	else if (EngineShowFlags.RayTracingDebug)
	{
		return VMI_RayTracingDebug;
	}
	else if (EngineShowFlags.VisualizeGPUSkinCache)
	{
		return VMI_VisualizeGPUSkinCache;
	}
	else if (EngineShowFlags.VisualizeLWCComplexity)
	{
		return VMI_LWCComplexity;
	}
	else if (EngineShowFlags.VisualizeShadowCasters)
	{
		return VMI_ShadowCasters;
	}
	else if (EngineShowFlags.Clay)
	{
		return VMI_Clay;
	}
	else if (EngineShowFlags.Zebra)
	{
		return VMI_Zebra;
	}
	else if (EngineShowFlags.FrontBackFace)
	{
		return VMI_FrontBackFace;
	}
	else if (EngineShowFlags.RandomColor)
	{
		return VMI_RandomColor;
	}

	return EngineShowFlags.Lighting ? VMI_Lit : VMI_Unlit;
}

const TCHAR* GetViewModeName(EViewModeIndex ViewModeIndex)
{
	switch (ViewModeIndex)
	{
		case VMI_Unknown:					return TEXT("Unknown");
		case VMI_BrushWireframe:			return TEXT("BrushWireframe");
		case VMI_Wireframe:					return TEXT("Wireframe");
		case VMI_Unlit:						return TEXT("Unlit");
		case VMI_Lit:						return TEXT("Lit");
		case VMI_Lit_DetailLighting:		return TEXT("Lit_DetailLighting");
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		case VMI_Lit_Wireframe:				return TEXT("Lit_Wireframe");
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
		case VMI_LightingOnly:				return TEXT("LightingOnly");
		case VMI_LightComplexity:			return TEXT("LightComplexity");
		case VMI_ShaderComplexity:			return TEXT("ShaderComplexity");
		case VMI_QuadOverdraw:				return TEXT("QuadOverdraw");
		case VMI_ShaderComplexityWithQuadOverdraw: return TEXT("ShaderComplexityWithQuadOverdraw");
		case VMI_PrimitiveDistanceAccuracy:	return TEXT("PrimitiveDistanceAccuracy");
		case VMI_MeshUVDensityAccuracy:		return TEXT("MeshUVDensityAccuracy");
		case VMI_MaterialTextureScaleAccuracy: return TEXT("MaterialTextureScaleAccuracy");
		case VMI_RequiredTextureResolution: return TEXT("RequiredTextureResolution");
		case VMI_StationaryLightOverlap:	return TEXT("StationaryLightOverlap");
		case VMI_LightmapDensity:			return TEXT("LightmapDensity");
		case VMI_LitLightmapDensity:		return TEXT("LitLightmapDensity");
		case VMI_ReflectionOverride:		return TEXT("ReflectionOverride");
		case VMI_VisualizeBuffer:			return TEXT("VisualizeBuffer");
		case VMI_VisualizeNanite:			return TEXT("VisualizeNanite");
		case VMI_VisualizeLumen:			return TEXT("VisualizeLumen");
		case VMI_VisualizeMegaLights:		return TEXT("VisualizeMegaLights");
		case VMI_VisualizeSubstrate:		return TEXT("VisualizeSubstrate");
		case VMI_VisualizeGroom:			return TEXT("VisualizeGroom");
		case VMI_VisualizeVirtualShadowMap:	return TEXT("VisualizeVirtualShadowMap");
		case VMI_VisualizeVirtualTexture:	return TEXT("VisualizeVirtualTexture");
		case VMI_RayTracingDebug:			return TEXT("RayTracingDebug");
		case VMI_PathTracing:				return TEXT("PathTracing");
		case VMI_CollisionPawn:				return TEXT("CollisionPawn");
		case VMI_CollisionVisibility:		return TEXT("CollisionVis");
		case VMI_LODColoration:				return TEXT("LODColoration");
		case VMI_HLODColoration:			return TEXT("HLODColoration");
		case VMI_VisualizeGPUSkinCache:		return TEXT("VisualizeGPUSkinCache");
		case VMI_LWCComplexity:				return TEXT("LWCComplexity");
		case VMI_ShadowCasters:				return TEXT("ShadowCasters");
		case VMI_Clay:						return TEXT("Clay");
		case VMI_Zebra:						return TEXT("Zebra");
		case VMI_FrontBackFace:				return TEXT("FrontBackFace");
		case VMI_RandomColor:				return TEXT("RandomColor");
		case VMI_StreamingTextureDeficit:	return TEXT("StreamingTextureDeficit");
		case VMI_StreamingTextureResidency:	return TEXT("StreamingTextureResidency");
		case VMI_StreamingMeshLODDeficit:	return TEXT("StreamingMeshLODDeficit");
		case VMI_StreamingMeshLODResidency:	return TEXT("StreamingMeshLODResidency");
	}
	return TEXT("");
}

struct FCustomShowFlagData
{
	FString			Name;
	FText			DisplayName;
	EShowFlagGroup	Group = SFG_Custom;
	IConsoleVariable* CVar = nullptr;
};

namespace CustomShowFlagsInternal
{
	TMap<FString, FEngineShowFlags::ECustomShowFlag>& GetNameToIndex()
	{
		static TMap<FString, FEngineShowFlags::ECustomShowFlag> NameToIndex;
		return NameToIndex;
	}

	TArray<FCustomShowFlagData>& GetRegisteredCustomShowFlags()
	{
		static TArray<FCustomShowFlagData> RegisteredCustomShowFlags;
		return RegisteredCustomShowFlags;
	}

	FTransactionallySafeRWLock& GetLock()
	{
		static FTransactionallySafeRWLock Lock;
		return Lock;
	}

	TBitArray<>& GetDefaultState()
	{
		static TBitArray<> DefaultState;
		return DefaultState;
	}
}


// Public custom show flags functions
FEngineShowFlags::ECustomShowFlag FEngineShowFlags::RegisterCustomShowFlag(const TCHAR* InName, bool DefaultEnabled, EShowFlagGroup Group, FText DisplayName)
{
	check(IsInGameThread()); // Have to register on game thread to make access to FEngineShowFlags::OnCustomShowFlagRegistered safe

	// Sanitize names, only keeping valid characters. Required for FEngineShowFlags::SetFromString() to work.
	const TCHAR* CurrentChar = InName;
	FString Name;
	while (*CurrentChar)
	{
		if (ensureAlwaysMsgf(IsValidNameChar(*CurrentChar), TEXT("Custom showflag \"%s\" contains invalid characters"), InName))
		{
			Name += *CurrentChar;
		}
		++CurrentChar;
	}

	FEngineShowFlags::ECustomShowFlag ReturnIndex;
	{
		const uint32 CurrentIndex = (uint32)FindIndexByName(*Name, nullptr);
		// Check if already exists
		if (CurrentIndex != INDEX_NONE)
		{
			if (ensureAlwaysMsgf(CurrentIndex >= SF_FirstCustom, TEXT("Attempted to register a custom showflag with the same name as an engine showflag: %s"), InName))
			{
				return (FEngineShowFlags::ECustomShowFlag)(CurrentIndex - SF_FirstCustom);
			}
			else
			{
				return ECustomShowFlag::None;
			}
		}

		UE::TRWScopeLock Lock(CustomShowFlagsInternal::GetLock(), SLT_Write);
		ReturnIndex = (FEngineShowFlags::ECustomShowFlag)CustomShowFlagsInternal::GetRegisteredCustomShowFlags().AddDefaulted(1);
		CustomShowFlagsInternal::GetDefaultState().Add(1);
		CustomShowFlagsInternal::GetNameToIndex().Add(Name, ReturnIndex);

		FCustomShowFlagData* Data = &CustomShowFlagsInternal::GetRegisteredCustomShowFlags()[(int32)ReturnIndex];
		Data->Name = MoveTemp(Name);
		Data->DisplayName = DisplayName.IsEmpty() ? FText::FromString(Data->Name) : DisplayName;
		Data->Group = Group;
		if (Data->CVar == nullptr)
		{
			Data->CVar = IConsoleManager::Get().RegisterConsoleVariable(*FString::Printf(TEXT("ShowFlag.%s"), *Data->Name), 2,
				TEXT("Allows to override a specific showflag (works in editor and game, \"show\" only works in game and UI only in editor)\n")
				TEXT("Useful to run a build many time with the same showflags (when put in consolevariables.ini like \"showflag.abc=0\")\n")
				TEXT(" 0: force the showflag to be OFF\n")
				TEXT(" 1: force the showflag to be ON\n")
				TEXT(" 2: do not override this showflag (default)"),
				ECVF_Default
			);
		}

		CustomShowFlagsInternal::GetDefaultState()[(int32)ReturnIndex] = DefaultEnabled;
	}

	FEngineShowFlags::OnCustomShowFlagRegistered.Broadcast();

	return ReturnIndex;
}

void FEngineShowFlags::IterateCustomFlags(TFunctionRef<bool(uint32, const FString&)> Functor)
{
	check(IsInGameThread());

	for (int32 Idx = 0; Idx < CustomShowFlagsInternal::GetRegisteredCustomShowFlags().Num(); ++Idx)
	{
		if (!Functor(Idx + SF_FirstCustom, CustomShowFlagsInternal::GetRegisteredCustomShowFlags()[Idx].Name))
		{
			return;
		}
	}
}

FSimpleMulticastDelegate FEngineShowFlags::OnCustomShowFlagRegistered;

// Private custom show flags functions
FEngineShowFlags::ECustomShowFlag FEngineShowFlags::FindCustomShowFlagByName(const FString& Name)
{
	UE::TRWScopeLock Lock(CustomShowFlagsInternal::GetLock(), SLT_ReadOnly);

	if (FEngineShowFlags::ECustomShowFlag* Existing = CustomShowFlagsInternal::GetNameToIndex().Find(Name))
	{
		return *Existing;
	}
	return ECustomShowFlag::None;
}

bool FEngineShowFlags::IsRegisteredCustomShowFlag(ECustomShowFlag Index)
{
	UE::TRWScopeLock Lock(CustomShowFlagsInternal::GetLock(), SLT_ReadOnly);
	return CustomShowFlagsInternal::GetRegisteredCustomShowFlags().IsValidIndex((uint32)Index);
}

FString FEngineShowFlags::GetCustomShowFlagName(ECustomShowFlag Index)
{
	UE::TRWScopeLock Lock(CustomShowFlagsInternal::GetLock(), SLT_ReadOnly);
	if (CustomShowFlagsInternal::GetRegisteredCustomShowFlags().IsValidIndex((uint32)Index))
	{
		return  CustomShowFlagsInternal::GetRegisteredCustomShowFlags()[(uint32)Index].Name;
	}
	return FString();
}

FText FEngineShowFlags::GetCustomShowFlagDisplayName(ECustomShowFlag Index)
{
	UE::TRWScopeLock Lock(CustomShowFlagsInternal::GetLock(), SLT_ReadOnly);
	if (CustomShowFlagsInternal::GetRegisteredCustomShowFlags().IsValidIndex((uint32)Index))
	{
		return CustomShowFlagsInternal::GetRegisteredCustomShowFlags()[(uint32)Index].DisplayName;
	}
	return FText();
}

EShowFlagGroup FEngineShowFlags::GetCustomShowFlagGroup(ECustomShowFlag Index)
{
	UE::TRWScopeLock Lock(CustomShowFlagsInternal::GetLock(), SLT_ReadOnly);
	if (CustomShowFlagsInternal::GetRegisteredCustomShowFlags().IsValidIndex((uint32)Index))
	{
		return CustomShowFlagsInternal::GetRegisteredCustomShowFlags()[(uint32)Index].Group;
	}
	return SFG_Normal;
}

void FEngineShowFlags::InitCustomShowFlags(EShowFlagInitMode InitMode)
{
	UE::TRWScopeLock Lock(CustomShowFlagsInternal::GetLock(), SLT_ReadOnly);
	if (InitMode == ESFIM_All0)
	{
		CustomShowFlags.Init(false, CustomShowFlagsInternal::GetDefaultState().Num());
	}
	else
	{
		CustomShowFlags = CustomShowFlagsInternal::GetDefaultState();
	}
}

void FEngineShowFlags::UpdateNewCustomShowFlags()
{
	UE::TRWScopeLock Lock(CustomShowFlagsInternal::GetLock(), SLT_ReadOnly);
	for (int32 i = CustomShowFlags.Num(); i < CustomShowFlagsInternal::GetDefaultState().Num(); ++i)
	{
		CustomShowFlags.Add(CustomShowFlagsInternal::GetDefaultState()[i]);
	}
}

bool FEngineShowFlags::FindCustomShowFlagDisplayName(const FString& Name, FText& OutText)
{
	UE::TRWScopeLock Lock(CustomShowFlagsInternal::GetLock(), SLT_ReadOnly);
	if (FEngineShowFlags::ECustomShowFlag* Existing = CustomShowFlagsInternal::GetNameToIndex().Find(Name))
	{
		OutText = CustomShowFlagsInternal::GetRegisteredCustomShowFlags()[(int32)*Existing].DisplayName;
		return true;
	}
	return false;
}

void FEngineShowFlags::EngineOverrideCustomShowFlagsFromCVars()
{
	UE::TRWScopeLock Lock(CustomShowFlagsInternal::GetLock(), SLT_ReadOnly);
	for (int32 i = 0; i < CustomShowFlagsInternal::GetRegisteredCustomShowFlags().Num(); ++i)
	{
		int32 Val = CustomShowFlagsInternal::GetRegisteredCustomShowFlags()[i].CVar->GetInt();
		if (Val != 2)
		{
			CustomShowFlags[i] = !!Val;
		}
	}
}
