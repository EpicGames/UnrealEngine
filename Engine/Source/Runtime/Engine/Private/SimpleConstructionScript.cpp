// Copyright Epic Games, Inc. All Rights Reserved.

#include "Engine/SimpleConstructionScript.h"
#include "Components/InputComponent.h"
#include "Engine/SCS_Node.h"
#include "EngineLogs.h"
#include "UObject/BlueprintsObjectVersion.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SimpleConstructionScript)

#if WITH_EDITOR
#include "Kismet2/CompilerResultsLog.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Kismet2/ComponentEditorUtils.h"
#include "Kismet2/Kismet2NameValidators.h"
#include "Logging/StructuredLog.h"
#else
#include "UObject/LinkerLoad.h"
#endif

//////////////////////////////////////////////////////////////////////////
// USimpleConstructionScript

// We append this suffix to template object names because the FObjectProperty we create at compile time will also be outered to the generated Blueprint class, and because we need cooking to be deterministic with respect to template object names.
const FString USimpleConstructionScript::ComponentTemplateNameSuffix(TEXT("_GEN_VARIABLE"));

#if WITH_EDITOR
namespace UE::Private
{
	void ConformArchetypeVisibility(UObject* Template)
	{
		if(!Template->HasAllFlags(RF_Public))
		{
			return;
		}

		TArray<UObject*> SubObjects;
		GetObjectsWithOuter(
			Template,
			SubObjects,
			EGetObjectsFlags::IncludeNestedObjects,
			RF_Transient,
			EInternalObjectFlags::Garbage);

		for (UObject* SubObject : SubObjects)
		{
			// In theory it might be good to propagate to all subobjects of
			// the template object, but I'm being conservative for now:
			if(SubObject->HasAnyFlags(RF_ArchetypeObject))
			{
				if(!SubObject->HasAllFlags(RF_Public))
				{
					UE_LOGFMT(LogBlueprint, Warning, "Conforming subobject flags: {subobject}", SubObject->GetPathName());
				}
				SubObject->SetFlags(RF_Public);
			}
		}
	}
}
#endif

USimpleConstructionScript::USimpleConstructionScript(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	DefaultSceneRootNode = nullptr;
	NameToSCSNodeMapRefCount = 0;
	ReregisterContext = nullptr;

#if WITH_EDITOR
	bIsConstructingEditorComponents = false;
#endif

	// Don't create a default scene root for the CDO and defer it for objects about to be loaded so we don't conflict with existing nodes
	if(!HasAnyFlags(RF_ClassDefaultObject|RF_NeedLoad))
	{
		ValidateSceneRootNodes();
	}
}

void USimpleConstructionScript::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

	Ar.UsingCustomVersion(FBlueprintsObjectVersion::GUID);

#if WITH_EDITORONLY_DATA
	if(Ar.IsLoading())
	{
		if(Ar.UEVer() < VER_UE4_REMOVE_NATIVE_COMPONENTS_FROM_BLUEPRINT_SCS)
		{
			// If we previously had a root node, we need to move it into the new RootNodes array. This is done in Serialize() in order to support SCS preloading (which relies on a valid RootNodes array).
			if(RootNode_DEPRECATED != NULL)
			{
				// Ensure it's been loaded so that its properties are valid
				RootNode_DEPRECATED->ConditionalPreload();

				// If the root node was not native
				if(!RootNode_DEPRECATED->bIsNative_DEPRECATED)
				{
					// Add the node to the root set
					RootNodes.Add(RootNode_DEPRECATED);
				}
				else
				{
					// For each child of the previously-native root node
					for (USCS_Node* Node : RootNode_DEPRECATED->GetChildNodes())
					{
						if(Node != NULL)
						{
							// Ensure it's been loaded (may not have been yet if we're preloading the SCS)
							Node->ConditionalPreload();

							// We only care about non-native child nodes (non-native nodes could only be attached to the root node in the previous version, so we don't need to examine native child nodes)
							if(!Node->bIsNative_DEPRECATED)
							{
								// Add the node to the root set
								RootNodes.Add(Node);

								// Set the previously-native root node as its parent component
								Node->bIsParentComponentNative = true;
								Node->ParentComponentOrVariableName = RootNode_DEPRECATED->NativeComponentName_DEPRECATED;
							}
						}
					}
				}

				// Clear the deprecated reference
				RootNode_DEPRECATED = NULL;
			}

			// Add any user-defined actor components to the root set
			for (USCS_Node* Node : ActorComponentNodes_DEPRECATED)
			{
				if(Node != NULL)
				{
					// Ensure it's been loaded (may not have been yet if we're preloading the SCS)
					Node->ConditionalPreload();
					if(!Node->bIsNative_DEPRECATED)
					{
						RootNodes.Add(Node);
					}
				}
			}

			// Clear the deprecated ActorComponent list
			ActorComponentNodes_DEPRECATED.Empty();
		}
	}
#endif // WITH_EDITORONLY_DATA
}

template <typename T>
class TSimpleConstructionScriptMutexScope
{
private:
	T& Mutex;
public:
	TSimpleConstructionScriptMutexScope(T& InMutex)
		: Mutex(InMutex)
	{
		if (!Mutex.TryLock())
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(SimpleConstructionScriptMutexContention);
			Mutex.Lock();
		}
	}

	~TSimpleConstructionScriptMutexScope()
	{
		Mutex.Unlock();
	}
};

void USimpleConstructionScript::PreloadChain()
{
	ConditionalPreload();

	TSimpleConstructionScriptMutexScope ScopeLock(Mutex);
	for (USCS_Node* Node : RootNodes)
	{
		Node->PreloadChain();
	}
}

void USimpleConstructionScript::PostLoad()
{
	Super::PostLoad();

	// Get the BlueprintGeneratedClass that owns the SCS
	const UClass* BPGeneratedClass = GetOwnerClass();

#if WITH_EDITOR
	// Skip fixup logic in the editor context if the owner class is already cooked.
	if (BPGeneratedClass && BPGeneratedClass->bCooked)
	{
		return;
	}

	// Get the Blueprint that owns the SCS
	UBlueprint* Blueprint = GetBlueprint();
	if (!Blueprint)
	{
		// sometimes the PostLoad can be called, after the object was trashed, we dont want this
		UE_LOGF(LogBlueprint, Warning, "USimpleConstructionScript::PostLoad() '%ls' cannot find its owner blueprint", *GetPathName());
		return;
	}
#endif // WITH_EDITOR

	// Protect AllNodes/RootNodes/DefaultSceneRootNode against concurrent reads from ALT
	// (via FindArchetype -> FindSCSNode/FindSCSNodeByGuid/PreloadChain).
	TSimpleConstructionScriptMutexScope ScopeLock(Mutex);

#if WITH_EDITOR
	// This pass is not needed during reinstancing.
	if (!GIsDuplicatingClassForReinstancing)
	{
		// Use a copy of the array for iterating, as we might have to reposition nodes in the hierarchy below (which can temporarily modify the array).
		TArray<USCS_Node*> LocalAllNodes = GetAllNodes();
		for (USCS_Node* Node : LocalAllNodes)
		{
			// Fix up any uninitialized category names
			if (Node->CategoryName.IsEmpty())
			{
				Node->CategoryName = NSLOCTEXT("SCS", "Default", "Default");
			}

			// Fix up components that may have switched from scene to non-scene type and vice-versa
			if (Node->ComponentTemplate != nullptr)
			{
				// Fix up any component template objects whose name doesn't match the current variable name; this ensures that there is always one unique template per node.
				FString VariableName = Node->GetVariableName().ToString();
				FString ComponentTemplateName = Node->ComponentTemplate->GetName();
				if (ComponentTemplateName.EndsWith(ComponentTemplateNameSuffix) && !ComponentTemplateName.StartsWith(VariableName))
				{
					Node->ComponentTemplate->ConditionalPostLoad();
					Node->ComponentTemplate = static_cast<UActorComponent*>(StaticDuplicateObject(Node->ComponentTemplate, Node->ComponentTemplate->GetOuter(), *(VariableName + ComponentTemplateNameSuffix)));
				}

				// Check to see if switched from scene to a non-scene component type
				if (!Node->ComponentTemplate->IsA<USceneComponent>())
				{
					// Otherwise, check to see if switched from scene to non-scene component type
					int32 RootNodeIndex = INDEX_NONE;
					if (!RootNodes.Find(Node, RootNodeIndex))
					{
						// Move the node into the root set if it's currently in the scene hierarchy
						USCS_Node* ParentNode = FindParentNode(Node);
						if (ParentNode != nullptr)
						{
							ParentNode->RemoveChildNode(Node);
						}

						RootNodes.Add(Node);
					}
					else
					{
						// Otherwise, if it's a root node, promote one of its children (if any) to take its place
						int32 PromoteIndex = FindPromotableChildNodeIndex(Node);
						if (PromoteIndex != INDEX_NONE)
						{
							// Remove it as a child node
							USCS_Node* ChildToPromote = Node->GetChildNodes()[PromoteIndex];
							Node->RemoveChildNodeAt(PromoteIndex, false);

							// Insert it as a root node just before its prior parent node; this way if it switches back to a scene type it won't supplant the new root we've just created
							RootNodes.Insert(ChildToPromote, RootNodeIndex);

							// Append previous root node's children to the new root
							ChildToPromote->MoveChildNodes(Node);

							// Copy any previous external attachment info from the previous root node
							ChildToPromote->bIsParentComponentNative = Node->bIsParentComponentNative;
							ChildToPromote->ParentComponentOrVariableName = Node->ParentComponentOrVariableName;
							ChildToPromote->ParentComponentOwnerClassName = Node->ParentComponentOwnerClassName;
						}

						// Clear info for any previous external attachment if set
						if (Node->ParentComponentOrVariableName != NAME_None)
						{
							Node->bIsParentComponentNative = false;
							Node->ParentComponentOrVariableName = NAME_None;
							Node->ParentComponentOwnerClassName = NAME_None;
						}
					}
				}
			}
		}
	}
#endif // WITH_EDITOR

	// Skip validation when reinstancing.
	if (!GIsDuplicatingClassForReinstancing)
	{
		// Fix up native/inherited parent attachments, in case anything has changed
		FixupRootNodeParentReferences();

		// Ensure that we have a valid scene root
		ValidateSceneRootNodes();
	}

	// Reset non-native "root" scene component scale values, prior to the change in which
	// we began applying custom scale values to root components at construction time. This
	// way older, existing Blueprint actor instances won't start unexpectedly getting scaled.
	if(GetLinkerUEVersion() < VER_UE4_BLUEPRINT_USE_SCS_ROOTCOMPONENT_SCALE)
	{
		if(BPGeneratedClass != nullptr)
		{
			// Get the Blueprint class default object
			AActor* CDO = Cast<AActor>(BPGeneratedClass->GetDefaultObject(false));
			if(CDO != NULL)
			{
				// Check for a native root component
				USceneComponent* NativeRootComponent = CDO->GetRootComponent();
				if(NativeRootComponent == nullptr)
				{
					// If no native root component exists, find the first non-native, non-parented SCS node with a
					// scene component template. This will be designated as the root component at construction time.
					for (USCS_Node* Node : RootNodes)
					{
						if(Node->ParentComponentOrVariableName == NAME_None)
						{
							// Note that we have to check for nullptr here, because it may be an ActorComponent type
							if (USceneComponent* SceneComponentTemplate = Cast<USceneComponent>(Node->ComponentTemplate))
							{
								const FVector ComponentRelativeScale3D = SceneComponentTemplate->GetRelativeScale3D();
								if (ComponentRelativeScale3D != FVector(1.0f, 1.0f, 1.0f))
								{
									UE_LOGF(LogBlueprint, Warning, "%ls: Found non-native root component custom scale for %ls (%ls) saved prior to being usable; reverting to default scale.", *BPGeneratedClass->GetName(), *Node->GetVariableName().ToString(), *ComponentRelativeScale3D.ToString());
									SceneComponentTemplate->SetRelativeScale3D_Direct(FVector(1.0f, 1.0f, 1.0f));
								}
							}

							// Done - no need to fix up any other nodes.
							break;
						}
					}
				}
			}
		}
	}

	if (GetLinkerUEVersion() < VER_UE4_SCS_STORES_ALLNODES_ARRAY)
	{
		// Fill out AllNodes if this is an older object
		if (RootNodes.Num() > 0)
		{
			AllNodes.Reset();
			for (USCS_Node* RootNode : RootNodes)
			{
				if (RootNode != nullptr)
				{
					AllNodes.Append(RootNode->GetAllNodes());
				}
			}
		}
	}
}

void USimpleConstructionScript::FixupSceneNodeHierarchy() 
{
#if WITH_EDITOR
	// determine the scene's root component, this isn't necessarily a node owned
	// by this SCS; it could be from a super SCS, or (if SceneRootNode and 
	// SceneRootComponentTemplate is not) it could be a native component
	USCS_Node* SceneRootNode = nullptr;
	USceneComponent* SceneRootComponentTemplate = GetSceneRootComponentTemplate(true, &SceneRootNode);

	if (SceneRootComponentTemplate == nullptr)
	{
		if (DefaultSceneRootNode && DefaultSceneRootNode->ComponentTemplate)
		{
			SceneRootNode = DefaultSceneRootNode;
			SceneRootComponentTemplate = CastChecked<USceneComponent>(DefaultSceneRootNode->ComponentTemplate);
			if (!RootNodes.Contains(SceneRootNode))
			{
				RootNodes.Add(SceneRootNode);
				AllNodes.Add(SceneRootNode);
			}
		}
		// if there is no scene root (then there shouldn't be anything but the 
		// default placeholder root).
		else
		{
			return;
		}
	}

	bool const bIsSceneRootNative = (SceneRootNode == nullptr);
	// cache this information before the mapper messes with the RootNode list
	bool const bThisOwnsSceneRoot = !bIsSceneRootNative && RootNodes.Contains(SceneRootNode);

	/** Helper struct which recursively maps the specified SCS hierarchy. */
	struct FSceneHierarchyMapper
	{
	public:
		FSceneHierarchyMapper(TArray<USCS_Node*>& RootNodesIn)
			: RootNodeList(RootNodesIn)
			, PendingParent(nullptr)
		{}

		/** Identifies orphan (root) nodes, and fixes up broken/cyclic tree linkages */
		void MapHierarchy(const TArray<USCS_Node*>& NodeList)
		{
			for (USCS_Node* Node : NodeList)
			{
				VisitNode(Node);
			}
		}

		/** Nests all orphans (and their nested hierarchies) under the target root */
		void FixupOrphanedNodes(USCS_Node* SceneRootNodeIn, USceneComponent* RootComponentTemplate, const bool bThisOwnsSceneRootIn)
		{
			bool bSkippedRootNode = false;
			for (USCS_Node* Orphan : OrphanedNodes)
			{
				if (Orphan == SceneRootNodeIn)
				{
					bSkippedRootNode = true;
					continue;
				}

				TArray<USCS_Node*>& RootNodeListRef = RootNodeList;
				auto AddToRootSet = [&RootNodeListRef](USCS_Node* Node)
				{
					int32 PreAddNum = RootNodeListRef.Num();
					RootNodeListRef.AddUnique(Node);

					// if it wasn't already in the RootSet, notify the user
					if (PreAddNum < RootNodeListRef.Num())
					{
						UE_LOGF(LogBlueprint, Warning, "Found orphaned component ('%ls') and added it to the Blueprint's root set. Please validate the component hierarchy is as wanted and resave.",
							*Node->GetVariableName().ToString());
					}
				};

				if (bThisOwnsSceneRootIn)
				{
					// Reparent to this BP's root node if it's still in the root set
					RootNodeList.Remove(Orphan);
					SceneRootNodeIn->AddChildNode(Orphan, false);
				}
				// if this field is filled out, assume it's set up to attach to  
				// a inherited component (unknown how to handle if that component is gone)
				else if (Orphan && Orphan->ParentComponentOrVariableName.IsNone())
				{
					if (SceneRootNodeIn == nullptr)
					{
						AddToRootSet(Orphan);
						// Parent to the native component template if not already attached
						Orphan->SetParent(RootComponentTemplate);
					}
					else
					{
						AddToRootSet(Orphan);
						// Parent to an inherited parent BP's node if not already attached
						Orphan->SetParent(SceneRootNodeIn);
					}
				} 
			}
			// make sure our root node is still in the root set
			check(!bThisOwnsSceneRootIn || bSkippedRootNode);
		}

	private:
		/** Recursively visits this node and its children (attempting to map the hierarchy) */
		bool VisitNode(USCS_Node* Node)
		{
			bool bPreviouslyVisited = false;
			VisitedNodes.Add(Node, &bPreviouslyVisited);

			if (bPreviouslyVisited)
			{
				// if we've visited this already, then we may be recursively 
				// traversing the tree, searching for broken link chains
				if (PendingParent && OrphanedNodes.Remove(Node))
				{
					FixupParentage(Node);
				}
				else
				{
					// we've visited this node before (and not as an orphan) - this  
					// indicates broken linkage (we've already identified it as 
					// belonging to another parent) - return false, so the parent 
					// will know to remove this from its children
					return false;
				}
			}
			else
			{
				auto VisitChildren = [this, Node]()
				{
					// recursively visit children so we can construct the hierarchy - iterate backwards so we can remove as we go
					for (int32 ChildIndex = Node->ChildNodes.Num() - 1; ChildIndex >= 0; --ChildIndex)
					{
						if (!VisitNode(Node->ChildNodes[ChildIndex]))
						{
							Node->ChildNodes.RemoveAt(ChildIndex);
						}
					}
				};

				if (UClass* ComponentClass = (Node->ComponentClass ? ToRawPtr(Node->ComponentClass) : (Node->ComponentTemplate ? Node->ComponentTemplate->GetClass() : nullptr)))
				{
					if (ComponentClass->IsChildOf<USceneComponent>())
					{
						// scoped for the following TGuardValue
						{
							TGuardValue<USCS_Node*> ParentStack(PendingParent, Node);
							VisitChildren();
						}

						// happens after recursing into children, so we don't add to 
						// the orphaned list till after children are querying it
						FixupParentage(Node);
					}
					else
					{
						RootNodeList.AddUnique(Node);
						if (Node->ChildNodes.Num() > 0)
						{
							// If this isn't a scene component but it has children, that's not good, so shift them to the pending parent or make them orphan nodes
							VisitChildren();

							Node->ChildNodes.Reset();
						}

						// A non-scene component should never be in the child list of someone else, so return false so the parent removes it from its list
						return false;
					}
				}
			}
			return true;
		}

		/** Nests the specified node under the active parent (if there isn't one pending, then it gets added to the orphan list - possibly removed later when we find the parent) */
		void FixupParentage(USCS_Node* Node)
		{
			if (PendingParent)
			{
				if (!ensure(Node->ParentComponentOrVariableName.IsNone() || Node->ParentComponentOrVariableName != PendingParent->GetVariableName()))
				{
					UE_LOGF(LogBlueprint, Warning, "Reparenting the '%ls' component (now nested under '%ls)' - possible cyclic linkage? Please validate the component hierarchy and resave the Blueprint.",
						*Node->GetVariableName().ToString(),
						*PendingParent->GetVariableName().ToString()
					);
				}
				PendingParent->AddChildNode(Node, /*bAddToAllNodes =*/false);
 
				if (RootNodeList.Remove(Node))
				{
					UE_LOGF(LogBlueprint, Warning, "The '%ls' component is being removed from the root set and nested under '%ls' - possible cyclic linkage? Please validate the component hierarchy and resave the Blueprint.",
						*Node->GetVariableName().ToString(),
						*PendingParent->GetVariableName().ToString()
					);
				}
				OrphanedNodes.Remove(Node);
			}
			else
			{
				// not necessarily an orphan, but waiting for us to parse its parent
				OrphanedNodes.Add(Node);
			}
		}

	private:
		TArray<USCS_Node*>& RootNodeList;
		TSet<USCS_Node*> VisitedNodes;
		TSet<USCS_Node*> OrphanedNodes;
		USCS_Node* PendingParent;
	};

	FSceneHierarchyMapper HierarchyMapper(MutableView(RootNodes));
	// identify orphan (root) nodes, and fixup cyclic hierarchies
	HierarchyMapper.MapHierarchy(AllNodes);
	// nest all orphaned nodes under the primary root node
	HierarchyMapper.FixupOrphanedNodes(SceneRootNode, SceneRootComponentTemplate, bThisOwnsSceneRoot);
#endif // #if WITH_EDITOR
}

void USimpleConstructionScript::FixupRootNodeParentReferences()
{
	// Get the BlueprintGeneratedClass that owns the SCS
	UClass* BPGeneratedClass = GetOwnerClass();
	if(BPGeneratedClass == NULL)
	{
		UE_LOGF(LogBlueprint, Warning, "USimpleConstructionScript::FixupRootNodeParentReferences() - owner class is NULL; skipping.");
		// cannot do the rest of fixup without a BPGC
		return;
	}

	for (int32 NodeIndex=0; NodeIndex < RootNodes.Num(); ++NodeIndex)
	{
		// If this root node is parented to a native/inherited component template
		USCS_Node* RootNode = RootNodes[NodeIndex];
		if(RootNode->ParentComponentOrVariableName != NAME_None)
		{
			bool bWasFound = false;

			// If the node is parented to a native component
			if(RootNode->bIsParentComponentNative)
			{
				// Get the Blueprint class default object
				AActor* CDO = Cast<AActor>(BPGeneratedClass->GetDefaultObject(false));
				if(CDO != NULL)
				{
					// Look for the parent component in the CDO's components array
					for (UActorComponent* ComponentTemplate : CDO->GetComponents())
					{
						if (ComponentTemplate && ComponentTemplate->GetFName() == RootNode->ParentComponentOrVariableName)
						{
							bWasFound = true;
							break;
						}
					}
				}
				else 
				{ 
					// SCS and BGClass depends on each other (while their construction).
					// Class is not ready, so one have to break the dependency circle.
					continue;
				}
			}
			// Otherwise the node is parented to an inherited SCS node from a parent Blueprint
			else
			{
				// Get the Blueprint hierarchy
				TArray<const UBlueprintGeneratedClass*> ParentBPClassStack;
				const bool bErrorFree = UBlueprintGeneratedClass::GetGeneratedClassesHierarchy(BPGeneratedClass, ParentBPClassStack);

				// Find the parent Blueprint in the hierarchy
				for(int32 StackIndex = ParentBPClassStack.Num() - 1; StackIndex > 0; --StackIndex)
				{
					const UBlueprintGeneratedClass* ParentClass = ParentBPClassStack[StackIndex];
					if( ParentClass != NULL
						&& ParentClass->SimpleConstructionScript != NULL
						&& ParentClass->GetFName() == RootNode->ParentComponentOwnerClassName)
					{
						// Attempt to locate a match by searching all the nodes that belong to the parent Blueprint's SCS
						for (USCS_Node* ParentNode : ParentClass->SimpleConstructionScript->GetAllNodes())
						{
							if (ParentNode != nullptr && ParentNode->GetVariableName() == RootNode->ParentComponentOrVariableName)
							{
								bWasFound = true;
								break;
							}
						}

						// We found a match; no need to continue searching the hierarchy
						break;
					}
				}
			}

			// Clear parent info if we couldn't find the parent component instance
			if(!bWasFound)
			{
				UE_LOGF(LogBlueprint, Warning, "USimpleConstructionScript::FixupRootNodeParentReferences() - Couldn't find %ls parent component '%ls' for '%ls' in BlueprintGeneratedClass '%ls' (it may have been removed)", RootNode->bIsParentComponentNative ? TEXT("native") : TEXT("inherited"), *RootNode->ParentComponentOrVariableName.ToString(), *RootNode->GetVariableName().ToString(), *BPGeneratedClass->GetName());

				RootNode->bIsParentComponentNative = false;
				RootNode->ParentComponentOrVariableName = NAME_None;
				RootNode->ParentComponentOwnerClassName = NAME_None;
			}
		}
	}

	// call this after we do the above ParentComponentOrVariableName fixup, 
	// because this operates differently for root nodes that have their 
	// ParentComponentOrVariableName field cleared
	//
	// repairs invalid scene hierarchies (like when this Blueprint has been 
	// reparented and there is no longer an inherited scene root... meaning one
	// of the scene component nodes here needs to be promoted)
	FixupSceneNodeHierarchy();
}

void USimpleConstructionScript::RegisterInstancedComponent(UActorComponent* InstancedComponent)
{
	// If this is a scene component, recursively register parent attachments within the actor's scene hierarchy first.
	if (USceneComponent* SceneComponent = Cast<USceneComponent>(InstancedComponent))
	{
		USceneComponent* ParentComponent = SceneComponent->GetAttachParent();
		if (ParentComponent != nullptr
			&& ParentComponent->GetOwner() == SceneComponent->GetOwner()
			&& !ParentComponent->IsRegistered())
		{
			RegisterInstancedComponent(ParentComponent);
		}
	}

	if (IsValid(InstancedComponent) && !InstancedComponent->IsRegistered() && InstancedComponent->bAutoRegister)
	{
		InstancedComponent->RegisterComponent();
	}
}

void USimpleConstructionScript::ExecuteScriptOnActor(AActor* Actor, const TInlineComponentArray<USceneComponent*>& NativeSceneComponents, const FTransform& RootTransform, const FRotationConversionCache* RootRelativeRotationCache, bool bIsDefaultTransform, ESpawnActorScaleMethod TransformScaleMethod)
{
	if(RootNodes.Num() > 0)
	{
		// Get the given actor's root component (can be NULL).
		USceneComponent* RootComponent = Actor->GetRootComponent();

		for (USCS_Node* RootNode : RootNodes)
		{
			// If the node is a default scene root and the actor already has a root component, skip it
			if (RootNode && ((RootNode != DefaultSceneRootNode) || (RootComponent == nullptr)))
			{
				// If the root node specifies that it has a parent
				USceneComponent* ParentComponent = nullptr;
				if(RootNode->ParentComponentOrVariableName != NAME_None)
				{
					// Get the Actor class object
					UClass* ActorClass = Actor->GetClass();
					check(ActorClass != nullptr);

					// If the root node is parented to a "native" component (i.e. in the 'NativeSceneComponents' array)
					if(RootNode->bIsParentComponentNative)
					{
						for(USceneComponent* NativeSceneComponent : NativeSceneComponents)
						{
							check(NativeSceneComponent != nullptr);

							// If we found a match, remember it
							if(NativeSceneComponent->GetFName() == RootNode->ParentComponentOrVariableName)
							{
								ParentComponent = NativeSceneComponent;
								break;
							}
						}
					}
					else
					{
						// In the non-native case, the SCS node's variable name property is used as the parent identifier
						FObjectPropertyBase* Property = FindFProperty<FObjectPropertyBase>(ActorClass, RootNode->ParentComponentOrVariableName);
						if(Property != nullptr)
						{
							// If we found a matching property, grab its value and use that as the parent for this node
							ParentComponent = Cast<USceneComponent>(Property->GetObjectPropertyValue_InContainer(Actor));
						}
					}
				}

				// Create the new component instance and any child components it may have
				RootNode->ExecuteNodeOnActor(Actor, ParentComponent != nullptr ? ParentComponent : RootComponent, &RootTransform, RootRelativeRotationCache, bIsDefaultTransform, TransformScaleMethod);
			}
		}
	}
	else if(Actor->GetRootComponent() == nullptr) // Must have a root component at the end of SCS, so if we don't have one already (from base class), create a SceneComponent now
	{
		USceneComponent* SceneComp = NewObject<USceneComponent>(Actor);

		// The object is new, so its safe for us to atomically set the flag in the open.
		UE_AUTORTFM_OPEN
			{
				SceneComp->SetFlags(RF_Transactional);
			};

		SceneComp->CreationMethod = EComponentCreationMethod::SimpleConstructionScript;
		if (RootRelativeRotationCache)
		{   // Enforces using the same rotator as much as possible.
			SceneComp->SetRelativeRotationCache(*RootRelativeRotationCache); 
		}
		SceneComp->SetWorldTransform(RootTransform);
		Actor->SetRootComponent(SceneComp);
		SceneComp->RegisterComponent();
	}
}

void USimpleConstructionScript::CreateNameToSCSNodeMap()
{
	if (NameToSCSNodeMapRefCount == 0)
	{
		const TArray<USCS_Node*>& Nodes = GetAllNodes();
		NameToSCSNodeMap.Reserve(Nodes.Num() * 2);

		for (USCS_Node* SCSNode : Nodes)
		{
			if (SCSNode)
			{
				NameToSCSNodeMap.Add(SCSNode->GetVariableName(), SCSNode);

				if (SCSNode->ComponentTemplate)
				{
					NameToSCSNodeMap.Add(SCSNode->ComponentTemplate->GetFName(), SCSNode);
				}
			}
		}
	}
	NameToSCSNodeMapRefCount++;
}

void USimpleConstructionScript::RemoveNameToSCSNodeMap()
{
	NameToSCSNodeMapRefCount--;
	check(NameToSCSNodeMapRefCount >= 0);

	if (NameToSCSNodeMapRefCount == 0)
	{
		NameToSCSNodeMap.Reset();
	}
}

#if WITH_EDITOR
UBlueprint* USimpleConstructionScript::GetBlueprint() const
{
	if (UClass* OwnerClass = GetOwnerClass())
	{
		return Cast<UBlueprint>(OwnerClass->ClassGeneratedBy);
	}
// >>> Backwards Compatibility:  VER_UE4_EDITORONLY_BLUEPRINTS
	if (UBlueprint* BP = Cast<UBlueprint>(GetOuter()))
	{
		return BP;
	}
// <<< End Backwards Compatibility
	return NULL;
}
#endif

#if WITH_EDITOR
void USimpleConstructionScript::SaveToTransactionBuffer()
{
	Modify();

	const TArray<USCS_Node*>& SCS_RootNodes = GetRootNodes();
	for (USCS_Node* Node : GetRootNodes())
	{
		if (Node)
		{
			Node->SaveToTransactionBuffer();
		}
	}
}
#endif //if WITH_EDITOR

UClass* USimpleConstructionScript::GetOwnerClass() const
{
	if (UClass* OwnerClass = Cast<UClass>(GetOuter()))
	{
		return OwnerClass;
	}
// >>> Backwards Compatibility:  VER_UE4_EDITORONLY_BLUEPRINTS
#if WITH_EDITOR
	if (UBlueprint* BP = Cast<UBlueprint>(GetOuter()))
	{
		return BP->GeneratedClass;
	}
#endif
// <<< End Backwards Compatibility
	return nullptr;
}

UClass* USimpleConstructionScript::GetParentClass() const
{
#if WITH_EDITOR
	if (UBlueprint* Blueprint = GetBlueprint())
	{
		return Blueprint->ParentClass;
	}
#endif
	if (UClass* OwnerClass = GetOwnerClass())
	{
		return OwnerClass->GetSuperClass();
	}

	return nullptr;
}

#if WITH_EDITOR
const TArray<USCS_Node*>& USimpleConstructionScript::GetAllNodes() const
{
	// Fill out AllNodes if this is an older object (should be from PostLoad but FindArchetype can happen earlier)
	if (RootNodes.Num() > 0 && AllNodes.Num() == 0)
	{
		USimpleConstructionScript* MutableThis = const_cast<USimpleConstructionScript*>(this);
		for (USCS_Node* RootNode : MutableThis->RootNodes)
		{
			if (RootNode != nullptr)
			{
				MutableThis->AllNodes.Append(RootNode->GetAllNodes());
			}
		}
	}

	return AllNodes;
}
#endif

void FSCSAllNodesHelper::Remove(USimpleConstructionScript* SCS, USCS_Node* SCSNode)
{
	SCS->Modify();
	SCS->AllNodes.Remove(SCSNode);
}

void FSCSAllNodesHelper::Add(USimpleConstructionScript* SCS, USCS_Node* SCSNode)
{
	SCS->Modify();
	SCS->AllNodes.Add(SCSNode);
}

void USimpleConstructionScript::AddNode(USCS_Node* Node)
{
	if(!RootNodes.Contains(Node))
	{
		Modify();

		RootNodes.Add(Node);
		AllNodes.Add(Node);

		ValidateSceneRootNodes();
	}
}

void USimpleConstructionScript::RemoveNode(USCS_Node* Node, const bool bValidateSceneRootNodes)
{
	// If it's a root node we are removing, clear it from the list
	if(RootNodes.Contains(Node))
	{
		Modify();

		RootNodes.Remove(Node);
		AllNodes.Remove(Node);

		Node->Modify();

		Node->bIsParentComponentNative = false;
		Node->ParentComponentOrVariableName = NAME_None;
		Node->ParentComponentOwnerClassName = NAME_None;

		if (bValidateSceneRootNodes)
		{
			ValidateSceneRootNodes();
		}
	}
	// Not the root, so iterate over all nodes looking for the one with us in its ChildNodes array
	else
	{
		USCS_Node* ParentNode = FindParentNode(Node);
		if(ParentNode != NULL)
		{
			ParentNode->RemoveChildNode(Node);
		}
	}
}

int32 USimpleConstructionScript::FindPromotableChildNodeIndex(USCS_Node* InParentNode) const
{
	int32 PromoteIndex = INDEX_NONE;

	if (InParentNode->GetChildNodes().Num() > 0)
	{
		PromoteIndex = 0;
		USCS_Node* Child = InParentNode->GetChildNodes()[PromoteIndex];

		// if this is an editor-only component, then it can't have any game-component children (better make sure that's the case)
		if (Child->ComponentTemplate != NULL && Child->ComponentTemplate->IsEditorOnly())
		{
			for (int32 ChildIndex = 1; ChildIndex < InParentNode->GetChildNodes().Num(); ++ChildIndex)
			{
				Child = InParentNode->GetChildNodes()[ChildIndex];
				// we found a game-component sibling, better make it the child to promote
				if (Child->ComponentTemplate != NULL && !Child->ComponentTemplate->IsEditorOnly())
				{
					PromoteIndex = ChildIndex;
					break;
				}
			}
		}
	}

	return PromoteIndex;
}

void USimpleConstructionScript::RemoveNodeAndPromoteChildren(USCS_Node* Node)
{
	Node->Modify();

	if (RootNodes.Contains(Node))
	{
		USCS_Node* ChildToPromote = nullptr;
		int32 PromoteIndex = FindPromotableChildNodeIndex(Node);
		if(PromoteIndex != INDEX_NONE)
		{
			ChildToPromote = Node->GetChildNodes()[PromoteIndex];
			Node->RemoveChildNodeAt(PromoteIndex, false);
		}

		Modify();

		if(ChildToPromote != NULL)
		{
			ChildToPromote->Modify();

			RootNodes.Add(ChildToPromote);
			ChildToPromote->MoveChildNodes(Node);

			ChildToPromote->bIsParentComponentNative = Node->bIsParentComponentNative;
			ChildToPromote->ParentComponentOrVariableName = Node->ParentComponentOrVariableName;
			ChildToPromote->ParentComponentOwnerClassName = Node->ParentComponentOwnerClassName;
		}
		
		RootNodes.Remove(Node);
		AllNodes.Remove(Node);

		Node->bIsParentComponentNative = false;
		Node->ParentComponentOrVariableName = NAME_None;
		Node->ParentComponentOwnerClassName = NAME_None;

		ValidateSceneRootNodes();
	}
	// Not the root so need to promote in place of node.
	else
	{
		USCS_Node* ParentNode = FindParentNode(Node);

		if (!ensure(ParentNode))
		{
#if WITH_EDITOR
			UE_LOGF(LogBlueprint, Error, "RemoveNodeAndPromoteChildren(%ls) failed to find a parent node in Blueprint %ls, attaching children to the root", *Node->GetName(), *GetBlueprint()->GetPathName());
#endif
			ParentNode = GetDefaultSceneRootNode();
		}

		check(ParentNode);
		if (ParentNode != nullptr)
		{
			ParentNode->Modify();

			// remove node and move children onto parent
			const int32 Location = ParentNode->GetChildNodes().Find(Node);
			ParentNode->RemoveChildNode(Node);
			ParentNode->MoveChildNodes(Node, Location);
		}
	}
}


USCS_Node* USimpleConstructionScript::FindParentNode(USCS_Node* InNode) const
{
	for(USCS_Node* TestNode : GetAllNodes())
	{
		if (TestNode && TestNode->GetChildNodes().Contains(InNode))
		{
			return TestNode;
		}
	}
	return nullptr;
}

USCS_Node* USimpleConstructionScript::FindSCSNode(const FName InName) const
{
	TSimpleConstructionScriptMutexScope ScopeLock(Mutex);

	if (NameToSCSNodeMap.Num() > 0)
	{
		return NameToSCSNodeMap.FindRef(InName);
	}

	for( USCS_Node* SCSNode : GetAllNodes() )
	{
		if (SCSNode && (SCSNode->GetVariableName() == InName || (SCSNode->ComponentTemplate && SCSNode->ComponentTemplate->GetFName() == InName)))
		{
			return SCSNode;
		}
	}
	return nullptr;
}

USCS_Node* USimpleConstructionScript::FindSCSNodeByGuid(const FGuid Guid) const
{
	TSimpleConstructionScriptMutexScope ScopeLock(Mutex);

	for (USCS_Node* SCSNode : GetAllNodes())
	{
		if (SCSNode && (SCSNode->VariableGuid == Guid))
		{
			return SCSNode;
		}
	}
	return nullptr;
}

#if WITH_EDITOR
USceneComponent* USimpleConstructionScript::GetSceneRootComponentTemplate(bool bShouldUseDefaultRoot, USCS_Node** OutSCSNode) const
{
	UClass* GeneratedClass = GetOwnerClass();
	UClass* ParentClass = GetParentClass();

	if(OutSCSNode)
	{
		*OutSCSNode = nullptr;
	}

	// Get the Blueprint class default object
	AActor* CDO = nullptr;
	if(GeneratedClass != nullptr)
	{
		CDO = Cast<AActor>(GeneratedClass->GetDefaultObject(false));
	}

	// If the generated class does not yet have a CDO, defer to the parent class
	if(CDO == nullptr && ParentClass != nullptr)
	{
		CDO = Cast<AActor>(ParentClass->GetDefaultObject(false));
	}

	// Check to see if we already have a native root component template
	USceneComponent* RootComponentTemplate = nullptr;
	if(CDO != nullptr)
	{
		// If the root component property is not set, the first available scene component will be used as the root. This matches what's done in the SCS editor.
		RootComponentTemplate = CDO->GetRootComponent();
		if(!RootComponentTemplate)
		{
			for (UActorComponent* Component : CDO->GetComponents())
			{
				if (USceneComponent* SceneComp = Cast<USceneComponent>(Component))
				{
					RootComponentTemplate = SceneComp;
					break;
				}
			}
		}
	}

	// Don't add the default scene root if we already have a native scene root component
	if(!RootComponentTemplate)
	{
		// Get the Blueprint hierarchy
		TArray<UBlueprintGeneratedClass*> BPStack;
		if (GeneratedClass)
		{
			UBlueprint::GetBlueprintHierarchyFromClass(GeneratedClass, BPStack);
		}
		else if (ParentClass)
		{
			UBlueprint::GetBlueprintHierarchyFromClass(ParentClass, BPStack);
		}

		// Note: Normally if the Blueprint has a parent, we can assume that the parent already has a scene root component set,
		// ...but we'll run through the hierarchy just in case there are legacy BPs out there that might not adhere to this assumption.
		TArray<const USimpleConstructionScript*> SCSStack;
		SCSStack.Reserve(BPStack.Num() + 1);

		SCSStack.Add(this);

		for(UBlueprintGeneratedClass* BPGC : BPStack)
		{
			if (BPGC && BPGC->SimpleConstructionScript)
			{
				SCSStack.AddUnique(BPGC->SimpleConstructionScript);
			}
		}

		// UBlueprint::GetBlueprintHierarchyFromClass returns first children then parents. So we need to revert the order.
		for (int32 StackIndex = SCSStack.Num() - 1; StackIndex >= 0 && !RootComponentTemplate; --StackIndex)
		{
			const TArray<USCS_Node*>& SCSRootNodes = SCSStack[StackIndex]->GetRootNodes();

			const bool bCanUseDefaultSceneRoot = bShouldUseDefaultRoot && DefaultSceneRootNode && DefaultSceneRootNode->ComponentTemplate && SCSRootNodes.Contains(DefaultSceneRootNode);
			// Check for any scene component nodes in the root set that are not the default scene root
			for (int32 RootNodeIndex = 0; RootNodeIndex < SCSRootNodes.Num() && RootComponentTemplate == nullptr; ++RootNodeIndex)
			{
				USCS_Node* RootNode = SCSRootNodes[RootNodeIndex];
				if (RootNode != nullptr
					&& RootNode != DefaultSceneRootNode
					&& RootNode->ComponentTemplate != nullptr
					&& RootNode->ComponentTemplate->IsA<USceneComponent>())
				{
					// if we found a non-default scene root, but the default scene root is also present, then we assume that's the desired one and still return null
					// this is to deal with the case where an actor component became scene root, but we don't want it to replace the default scene root if that was
					// deliberately being used
					if (bCanUseDefaultSceneRoot)
					{
						return nullptr;
					}

					if (OutSCSNode)
					{
						*OutSCSNode = RootNode;
					}

					RootComponentTemplate = Cast<USceneComponent>(RootNode->ComponentTemplate);
				}
			}
		}
	}

	return RootComponentTemplate;
}
#endif

void USimpleConstructionScript::ValidateSceneRootNodes()
{
#if WITH_EDITOR
	TSimpleConstructionScriptMutexScope ScopeLock(Mutex);

	UBlueprint* Blueprint = GetBlueprint();

	if(DefaultSceneRootNode == nullptr)
	{
		// If applicable, create a default scene component node
		if(Blueprint != nullptr
			&& FBlueprintEditorUtils::IsActorBased(Blueprint)
			&& Blueprint->BlueprintType != BPTYPE_MacroLibrary)
		{
			DefaultSceneRootNode = CreateNode(USceneComponent::StaticClass(), USceneComponent::GetDefaultSceneRootVariableName());
			CastChecked<USceneComponent>(DefaultSceneRootNode->ComponentTemplate)->bVisualizeComponent = true;
		}
	}

	if(DefaultSceneRootNode != nullptr)
	{
		// Get the current root component template
		const USceneComponent* RootComponentTemplate = GetSceneRootComponentTemplate();

		// Add the default scene root back in if there are no other scene component nodes that can be used as root; otherwise, remove it
		if(RootComponentTemplate == nullptr
			&& !RootNodes.Contains(DefaultSceneRootNode))
		{
			RootNodes.Add(DefaultSceneRootNode);
			AllNodes.Add(DefaultSceneRootNode);
		}
		else if(RootComponentTemplate != nullptr
			&& RootNodes.Contains(DefaultSceneRootNode))
		{
			// If the default scene root has any child nodes, determine what they should parent to.
			USCS_Node* RootNode = nullptr;
			bool bIsParentComponentNative = false;
			FName ParentComponentOrVariableName = NAME_None;
			FName ParentComponentOwnerClassName = NAME_None;
			if(UBlueprintGeneratedClass* BPClass = Cast<UBlueprintGeneratedClass>(RootComponentTemplate->GetOuter()))
			{
				// The root scene component is an SCS node.
				if(BPClass->SimpleConstructionScript != nullptr)
				{
					const TArray<USCS_Node*> SCSRootNodes = BPClass->SimpleConstructionScript->GetRootNodes();
					for(USCS_Node* SCSNode : SCSRootNodes)
					{
						if(SCSNode != nullptr && SCSNode->ComponentTemplate == RootComponentTemplate)
						{
							if(BPClass->SimpleConstructionScript != this)
							{
								// The root node is inherited from a parent BP class.
								ParentComponentOwnerClassName = BPClass->GetFName();
								ParentComponentOrVariableName = SCSNode->GetVariableName();
							}
							else
							{
								// The root node belongs to the current BP class.
								RootNode = SCSNode;
							}
							
							break;
						}
					}
				}
			}
			else
			{
				// The root scene component is a native component.
				bIsParentComponentNative = true;
				ParentComponentOrVariableName = RootComponentTemplate->GetFName();
			}

			// Reparent any child nodes within the current hierarchy.
			for(USCS_Node* ChildNode : DefaultSceneRootNode->ChildNodes)
			{
				if(RootNode != nullptr)
				{
					// We have an existing root node within the current BP class.
					RootNode->AddChildNode(ChildNode, false);
				}
				else
				{
					// The current root node is inherited from a parent class (may be BP or native).
					RootNodes.Add(ChildNode);
					ChildNode->bIsParentComponentNative = bIsParentComponentNative;
					ChildNode->ParentComponentOrVariableName = ParentComponentOrVariableName;
					ChildNode->ParentComponentOwnerClassName = ParentComponentOwnerClassName;
				}
			}

			// Remove the default scene root node from the current hierarchy.
			RootNodes.Remove(DefaultSceneRootNode);
			AllNodes.Remove(DefaultSceneRootNode);
			DefaultSceneRootNode->ChildNodes.Empty();

			// These shouldn't be set, but just in case...
			DefaultSceneRootNode->bIsParentComponentNative = false;
			DefaultSceneRootNode->ParentComponentOrVariableName = NAME_None;
			DefaultSceneRootNode->ParentComponentOwnerClassName = NAME_None;
		}
	}
#endif // WITH_EDITOR
}

#if WITH_EDITOR
EDataValidationResult USimpleConstructionScript::IsDataValid(FDataValidationContext& Context) const
{
	EDataValidationResult Result = Super::IsDataValid(Context);
	Result = (Result == EDataValidationResult::NotValidated) ? EDataValidationResult::Valid : Result;

	for (USCS_Node* Node : RootNodes)
	{
		if (Node)
		{
			EDataValidationResult NodeResult = Node->IsDataValid(Context);
			Result = CombineDataValidationResults(Result, NodeResult);
		}
	}
	return Result;
}

void USimpleConstructionScript::GenerateListOfExistingNames(TSet<FName>& CurrentNames) const
{
	const UBlueprintGeneratedClass* OwnerClass = Cast<const UBlueprintGeneratedClass>(GetOuter());
	const UBlueprint* Blueprint = Cast<const UBlueprint>(OwnerClass ? OwnerClass->ClassGeneratedBy : NULL);
	// >>> Backwards Compatibility:  VER_UE4_EDITORONLY_BLUEPRINTS
	if (!Blueprint)
	{
		Blueprint = Cast<UBlueprint>(GetOuter());
	}
	// <<< End Backwards Compatibility
	check(Blueprint);

	ForEachObjectWithOuter(Blueprint->GeneratedClass, [&CurrentNames](UObject* BlueprintClassChild)
	{
		CurrentNames.Add(BlueprintClassChild->GetFName());
	});	
	
	UClass* FirstNativeClass = FBlueprintEditorUtils::FindFirstNativeClass(Blueprint->ParentClass);

	ForEachObjectWithOuter(FirstNativeClass->GetDefaultObject(), [&CurrentNames](UObject* NativeCDOChild)
	{
		CurrentNames.Add(NativeCDOChild->GetFName());
	});

	if (Blueprint->SkeletonGeneratedClass)
	{
		// First add the class variables.
		FBlueprintEditorUtils::GetClassVariableList(Blueprint, CurrentNames, true);
		// Then the function names.
		FBlueprintEditorUtils::GetFunctionNameList(Blueprint, CurrentNames);
	}

	// And add their names
	for (const USCS_Node* ChildNode : GetAllNodes())
	{
		if (ChildNode)
		{
			const FName VariableName = ChildNode->GetVariableName();
			if (VariableName != NAME_None)
			{
				CurrentNames.Add(VariableName);
			}
		}
	}

	if (GetDefaultSceneRootNode())
	{
		CurrentNames.Add(GetDefaultSceneRootNode()->GetVariableName());
	}
}

FName USimpleConstructionScript::GenerateNewComponentName(const UClass* ComponentClass, FName DesiredName ) const
{
	TSet<FName> CurrentNames;
	GenerateListOfExistingNames(CurrentNames);

	FName NewName;
	if (ComponentClass)
	{
		if (DesiredName != NAME_None && !CurrentNames.Contains(DesiredName))
		{
			NewName = DesiredName;
		}
		else
		{
			FString ComponentName;
			int32 Counter = 1;

			auto BuildNewName = [&Counter, &ComponentName]()
			{
				return FName(*(FString::Printf(TEXT("%s%d"), *ComponentName, Counter++)));
			};

			if (DesiredName != NAME_None)
			{
				ComponentName = DesiredName.ToString();

				// If a desired name is supplied then walk back and find any numeric suffix so we can increment it nicely
				int32 Index = ComponentName.Len();
				while (Index > 0 && ComponentName[Index-1] >= '0' && ComponentName[Index-1] <= '9')
				{
					--Index;
				}

				if (Index < ComponentName.Len())
				{
					FString NumericSuffix = ComponentName.RightChop(Index);
					Counter = FCString::Atoi(*NumericSuffix);
					NumericSuffix = FString::Printf(TEXT("%d"), Counter); // Restringify the counter to account for leading 0s that we don't want to remove
					ComponentName.RemoveAt(ComponentName.Len() - NumericSuffix.Len(), NumericSuffix.Len(), EAllowShrinking::No);
					++Counter;
					NewName = BuildNewName();
				}
				else
				{
					NewName = DesiredName;
				}
			}
			else
			{
				ComponentName = ComponentClass->GetName();

				if (!ComponentClass->HasAnyClassFlags(CLASS_CompiledFromBlueprint))
				{
					ComponentName.RemoveFromEnd(TEXT("Component"));
				}
				else
				{
					ComponentName.RemoveFromEnd("_C");
				}
				NewName = *ComponentName;
			}

			while (CurrentNames.Contains(NewName))
			{
				NewName = BuildNewName();
			}
		}
	}
	return NewName;
}

USCS_Node* USimpleConstructionScript::CreateNodeImpl(UActorComponent* NewComponentTemplate, FName ComponentVariableName)
{
	USCS_Node* NewNode = NewObject<USCS_Node>(this, MakeUniqueObjectName(this, USCS_Node::StaticClass()));
	NewNode->SetFlags(RF_Transactional);
	NewNode->ComponentClass = NewComponentTemplate->GetClass();
	NewNode->ComponentTemplate = NewComponentTemplate;
	NewNode->SetVariableName(ComponentVariableName, false);

	// Note: This should match up with UEdGraphSchema_K2::VR_DefaultCategory
	NewNode->CategoryName = NSLOCTEXT("SCS", "Default", "Default");
	NewNode->VariableGuid = FGuid::NewGuid();
	return NewNode;
}

// Evicts any orphaned _GEN_VARIABLE object that occupies VariableName's slot in TargetOuter (e.g. a
// template left behind by an incomplete undo/redo), then renames ComponentTemplate into that slot.
static void RenameComponentTemplateIntoClass(UActorComponent* ComponentTemplate, UObject* TargetOuter, FName VariableName)
{
	const FString FullTemplateName = VariableName.ToString() + USimpleConstructionScript::ComponentTemplateNameSuffix;
	UObject* Collision = FindObject<UObject>(TargetOuter, *FullTemplateName);
	while (Collision)
	{
		Collision->Rename(nullptr, GetTransientPackage(), REN_DoNotDirty | REN_DontCreateRedirectors | REN_AllowPackageLinkerMismatch);
		Collision = FindObject<UObject>(TargetOuter, *FullTemplateName);
	}
	ComponentTemplate->Rename(*FullTemplateName, TargetOuter, REN_DoNotDirty | REN_DontCreateRedirectors | REN_AllowPackageLinkerMismatch);
}

USCS_Node* USimpleConstructionScript::CreateNode(UClass* NewComponentClass, FName NewComponentVariableName)
{
	UBlueprint* Blueprint = GetBlueprint();
	check(Blueprint);
	check(NewComponentClass->IsChildOf(UActorComponent::StaticClass()));
	ensure(Cast<UBlueprintGeneratedClass>(Blueprint->GeneratedClass));

	NewComponentVariableName = GenerateNewComponentName(NewComponentClass, NewComponentVariableName);

	// At this point we should have a unique, explicit name to use for the template object.
	check(NewComponentVariableName != NAME_None);

	// A bit of a hack, but by doing this we ensure that the original object isn't outered to the BPGC. That way if we undo this action later, it'll rename the template away from the BPGC.
	// This is necessary because of our template object naming scheme that's in place to ensure deterministic cooking. We have to keep the SCS node and template object names in sync as a result,
	// and leaving the template outered to the BPGC can lead to template object name collisions when attempting to rename the remaining SCS nodes. See USCS_Node::NameWasModified() for more details.
	UActorComponent* NewComponentTemplate = NewObject<UActorComponent>(GetTransientPackage(), NewComponentClass, NAME_None, RF_ArchetypeObject | RF_Transactional | RF_Public);

	// Record initial object state in case we're in a transaction context.
	NewComponentTemplate->Modify();

	RenameComponentTemplateIntoClass(NewComponentTemplate, Blueprint->GeneratedClass, NewComponentVariableName);

	return CreateNodeImpl(NewComponentTemplate, NewComponentVariableName);
}

USCS_Node* USimpleConstructionScript::CreateNodeAndRenameComponent(UActorComponent* NewComponentTemplate)
{
	check(NewComponentTemplate);

	// When copying and pasting we'd prefer to keep the component name
	// However, the incoming template will have the template name suffix on it so
	// acquire the desired name by stripping the suffix
	FName DesiredName;
	FString TemplateName = NewComponentTemplate->GetName();
	if (TemplateName.EndsWith(ComponentTemplateNameSuffix))
	{
		DesiredName = *TemplateName.LeftChop(ComponentTemplateNameSuffix.Len());
	}
	FName NewComponentVariableName = GenerateNewComponentName(NewComponentTemplate->GetClass(), DesiredName);

	// At this point we should have a unique, explicit name to use for the template object.
	check(NewComponentVariableName != NAME_None);

	// Relocate the instance from the transient package to the BPGC and assign it a unique object name,
	// evicting any orphaned template that may occupy the slot.
	RenameComponentTemplateIntoClass(NewComponentTemplate, GetBlueprint()->GeneratedClass, NewComponentVariableName);

	return CreateNodeImpl(NewComponentTemplate, NewComponentVariableName);
}

void USimpleConstructionScript::ValidateNodeVariableNames(FCompilerResultsLog& MessageLog)
{
	UBlueprint* Blueprint = GetBlueprint();
	if (!ensureMsgf(Blueprint != nullptr, TEXT("Cannot validate SCS node variable names because the owning Blueprint could not be determined from the SCS context (perhaps it was deleted?).")))
	{
		return;
	}

	TSharedPtr<FKismetNameValidator> ParentBPNameValidator;
	if( Blueprint->ParentClass != NULL )
	{
		UBlueprint* ParentBP = Cast<UBlueprint>(Blueprint->ParentClass->ClassGeneratedBy);
		if( ParentBP != NULL )
		{
			ParentBPNameValidator = MakeShareable(new FKismetNameValidator(ParentBP));
		}
	}

	TSharedPtr<FKismetNameValidator> CurrentBPNameValidator = MakeShareable(new FKismetNameValidator(Blueprint));

	int32 Counter=0;

	for (USCS_Node* Node : GetAllNodes())
	{
		if( Node && Node->ComponentTemplate && Node != DefaultSceneRootNode )
		{
			FName VariableName = Node->GetVariableName();

			// Replace missing or invalid component variable names
			if( VariableName == NAME_None
#if WITH_EDITORONLY_DATA
				|| Node->bVariableNameAutoGenerated_DEPRECATED
#endif
				|| !FComponentEditorUtils::IsValidVariableNameString(Node->ComponentTemplate, VariableName.ToString()) )
			{
				FName OldName = VariableName;

				// Generate a new default variable name for the component.
				VariableName = GenerateNewComponentName(Node->ComponentTemplate->GetClass());
				Node->SetVariableName(VariableName);
#if WITH_EDITORONLY_DATA
				Node->bVariableNameAutoGenerated_DEPRECATED = false;
#endif

				if( OldName != NAME_None )
				{
					FBlueprintEditorUtils::ReplaceVariableReferences(Blueprint, OldName, VariableName);

					MessageLog.Warning(*FString::Printf(TEXT("Found a component variable with an invalid name (%s) - changed to %s."), *OldName.ToString(), *VariableName.ToString()));
				}
			}
			else if( ParentBPNameValidator.IsValid() && ParentBPNameValidator->IsValid(VariableName) != EValidatorResult::Ok )
			{
				FName OldName = VariableName;

				VariableName = FBlueprintEditorUtils::FindUniqueKismetName(Blueprint, OldName.ToString());
				FBlueprintEditorUtils::RenameMemberVariable(Blueprint, OldName, VariableName );

				MessageLog.Warning(*FString::Printf(TEXT("Found a component variable with a conflicting name (%s) - changed to %s."), *OldName.ToString(), *VariableName.ToString()));
			}
		}
	}
}

void USimpleConstructionScript::ValidateNodeTemplates(FCompilerResultsLog& MessageLog)
{
	TArray<USCS_Node*> Nodes = GetAllNodes();
	UClass* OwningClass = GetOwnerClass();

	// it's likely that we shouldn't run any of this for diff loaded packages, but don't 
	// want to destabilize anyone.. so just soldering off my new logic
	const bool bForDiff = OwningClass->GetOutermost()->HasAnyPackageFlags(PKG_ForDiffing);

	for (USCS_Node* Node : Nodes)
	{
		if (GetLinkerUEVersion() < VER_UE4_REMOVE_INPUT_COMPONENTS_FROM_BLUEPRINTS)
		{
			if (!Node->bIsNative_DEPRECATED && Node->ComponentTemplate && Node->ComponentTemplate->IsA<UInputComponent>())
			{
				RemoveNodeAndPromoteChildren(Node);
			}
		}

		// Couldn't find the template - the Blueprint or C++ class may have been deleted out from under us, or it was not loaded due to client/server exclusion
		if (Node->ComponentTemplate == nullptr)
		{
			bool bRemoveNode = true;
			if (Node->ComponentClass != nullptr)
			{
				// Don't remove the node if the template was not loaded due to client/server exclusion (i.e. if we can't instance the class within the current runtime context)
				UObject* ComponentCDO = Node->ComponentClass->GetDefaultObject();
				bRemoveNode = UObject::CanCreateInCurrentContext(ComponentCDO);
			}
			else
			{
				UBlueprint* Blueprint = GetBlueprint();
				const FString BlueprintName = Blueprint != nullptr ? Blueprint->GetName() : FString();
				MessageLog.Warning(*FString::Printf(TEXT("Component class is not set for '%s' - this component will not be instanced, and additional warnings or errors may occur when compiling Blueprint '%s'."), *Node->GetVariableName().ToString(), *BlueprintName));

				if (GetLinkerCustomVersion(FBlueprintsObjectVersion::GUID) < FBlueprintsObjectVersion::SCSHasComponentTemplateClass
					&& (IsRunningDedicatedServer() || IsRunningClientOnly()))
				{
					const FString BlueprintPathName = Blueprint != nullptr ? Blueprint->GetPathName() : FString();
					MessageLog.Note(*FString::Printf(TEXT("Try launching the editor and resaving '%s' in order to fix this."), *BlueprintPathName));
				}
			}

			if (bRemoveNode)
			{
				RemoveNodeAndPromoteChildren(Node);
			}
		}
		else if (!bForDiff)
		{
			if (Node->ComponentTemplate->GetOuter() != OwningClass)
			{
				// this component template is somehow not owned by the class, recreate it:
				FString VariableName = Node->GetVariableName().ToString();
				if (Node->ComponentTemplate->HasAnyFlags(RF_ClassDefaultObject))
				{
					// duplicate won't work on CDOs because SDO will (for no good reason) reset loaders when duplicating a CDO...
					Node->ComponentTemplate = NewObject<UActorComponent>(
						OwningClass, 
						Node->ComponentTemplate->GetClass(), 
						*(VariableName + ComponentTemplateNameSuffix), 
						RF_ArchetypeObject | RF_Transactional | RF_Public);
				}
				else
				{
					Node->ComponentTemplate = static_cast<UActorComponent*>(StaticDuplicateObject(
						Node->ComponentTemplate, 
						OwningClass, 
						*(VariableName + ComponentTemplateNameSuffix)));
					Node->ComponentTemplate->SetFlags(RF_ArchetypeObject | RF_Transactional | RF_Public);
				}
				MessageLog.Warning(*FText::Format(NSLOCTEXT(
					"SimpleConstructionScript", "CorruptComponentFixed", 
					"SCS Node component template {0} has been recreated because it was unexpectedly not owned by the class"), FText::FromString(VariableName)).ToString());
			}
			
#if WITH_EDITOR
			UE::Private::ConformArchetypeVisibility(Node->ComponentTemplate);
#endif//WITH_EDITOR
		}
	}
}

void USimpleConstructionScript::ClearEditorComponentReferences()
{
	for (USCS_Node* Node : GetAllNodes())
	{
		if (Node)
		{
			Node->EditorComponentInstance = NULL;
		}
	}
}

void USimpleConstructionScript::BeginEditorComponentConstruction()
{
	if(!bIsConstructingEditorComponents)
	{
		ClearEditorComponentReferences();

		bIsConstructingEditorComponents = true;
	}
}

void USimpleConstructionScript::EndEditorComponentConstruction()
{
	bIsConstructingEditorComponents = false;
}
#endif

