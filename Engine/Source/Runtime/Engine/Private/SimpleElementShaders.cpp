// Copyright Epic Games, Inc. All Rights Reserved.

/*==============================================================================
	SimpleElementShaders.h: Definitions for simple element shaders.
==============================================================================*/

#include "SimpleElementShaders.h"
#include "Engine/EngineTypes.h"
#include "ShaderParameterUtils.h"
#include "Misc/DelayedAutoRegister.h"
#include "SceneView.h"
#include "RHIContext.h"
#include "SceneRelativeViewMatrices.h"
#include "DataDrivenShaderPlatformInfo.h"

/*------------------------------------------------------------------------------
	Simple element vertex shader.
------------------------------------------------------------------------------*/

FSimpleElementVS::FSimpleElementVS(const ShaderMetaType::CompiledShaderInitializerType& Initializer):
	FGlobalShader(Initializer)
{
	RelativeTransform.Bind(Initializer.ParameterMap,TEXT("Transform"), SPF_Mandatory);
	TransformPositionHigh.Bind(Initializer.ParameterMap, TEXT("TransformPositionHigh"), SPF_Optional); // TransformTilePosition may be optimized out if LWC is disabled
}

void FSimpleElementVS::SetParameters(FRHIBatchedShaderParameters& BatchedParameters, const FMatrix& WorldToClipMatrix)
{
	const FMatrix44f FinalMats[2] = { FMatrix44f(WorldToClipMatrix) , FMatrix44f(WorldToClipMatrix) };
	const FVector4f PositionsHigh[2] = { FVector3f::ZeroVector, FVector3f::ZeroVector };
	SetShaderValue(BatchedParameters, RelativeTransform, FinalMats);
	SetShaderValue(BatchedParameters, TransformPositionHigh, PositionsHigh);
}

void FSimpleElementVS::SetParameters(FRHIBatchedShaderParameters& BatchedParameters, const FDFRelativeViewMatrices& Matrices, const FDFRelativeViewMatrices& InstancedViewMatrices)
{
	const FMatrix44f FinalMats[2] = { Matrices.RelativeWorldToClip , InstancedViewMatrices.RelativeWorldToClip };
	const FVector4f PositionsHigh[2] = { Matrices.PositionHigh, InstancedViewMatrices.PositionHigh };
	SetShaderValue(BatchedParameters, RelativeTransform, FinalMats);
	SetShaderValue(BatchedParameters, TransformPositionHigh, PositionsHigh);
}

void FSimpleElementVS::ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
{
	FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
	OutEnvironment.SetDefine(TEXT("ENABLE_LWC"), 1); // Currently SimpleElementVertexShader.usf is shared with FCubemapTexturePropertiesVS, so need separate paths for LWC vs non-LWC
}

/*------------------------------------------------------------------------------
	Simple element pixel shaders.
------------------------------------------------------------------------------*/

FSimpleElementPS::FSimpleElementPS(const ShaderMetaType::CompiledShaderInitializerType& Initializer) :
	FGlobalShader(Initializer)
{
	InTexture.Bind(Initializer.ParameterMap,TEXT("InTexture"));
	InTextureSampler.Bind(Initializer.ParameterMap,TEXT("InTextureSampler"));
	TextureComponentReplicate.Bind(Initializer.ParameterMap,TEXT("TextureComponentReplicate"));
	TextureComponentReplicateAlpha.Bind(Initializer.ParameterMap,TEXT("TextureComponentReplicateAlpha"));
}

void FSimpleElementPS::SetEditorCompositingParameters(FRHIBatchedShaderParameters& BatchedParameters, const FSceneView* View)
{
	if (View)
	{
		FGlobalShader::SetParameters<FViewUniformShaderParameters>(BatchedParameters, View->ViewUniformBuffer);
	}
	else
	{
		// Unset the view uniform buffers since we don't have a view
		SetUniformBufferParameter(BatchedParameters, GetUniformBufferParameter<FViewUniformShaderParameters>(), nullptr);
	}
}

void FSimpleElementPS::SetParameters(FRHIBatchedShaderParameters& BatchedParameters, const FTexture* TextureValue)
{
	SetTextureParameter(BatchedParameters, InTexture, InTextureSampler, TextureValue);

	SetShaderValue(BatchedParameters, TextureComponentReplicate, TextureValue->bGreyScaleFormat ? FLinearColor(1, 0, 0, 0) : FLinearColor(0, 0, 0, 0));
	SetShaderValue(BatchedParameters, TextureComponentReplicateAlpha, TextureValue->bGreyScaleFormat ? FLinearColor(1, 0, 0, 0) : FLinearColor(0, 0, 0, 1));
}

void FSimpleElementPS::SetParameters(FRHIBatchedShaderParameters& BatchedParameters, const FSceneView* View, const FTexture* TextureValue)
{
	SetEditorCompositingParameters(BatchedParameters, View);
	SetParameters(BatchedParameters, TextureValue);
}

FSimpleElementAlphaOnlyPS::FSimpleElementAlphaOnlyPS(const ShaderMetaType::CompiledShaderInitializerType& Initializer) :
	FSimpleElementPS(Initializer)
{
}

FSimpleElementGammaBasePS::FSimpleElementGammaBasePS(const ShaderMetaType::CompiledShaderInitializerType& Initializer) :
	FSimpleElementPS(Initializer)
{
	Gamma.Bind(Initializer.ParameterMap,TEXT("Gamma"));
}

void FSimpleElementGammaBasePS::SetParameters(FRHIBatchedShaderParameters& BatchedParameters, const FTexture* Texture, float GammaValue, ESimpleElementBlendMode BlendMode)
{
	FSimpleElementPS::SetParameters(BatchedParameters, Texture);
	SetShaderValue(BatchedParameters, Gamma, GammaValue);
}

void FSimpleElementGammaBasePS::SetParameters(FRHIBatchedShaderParameters& BatchedParameters, const FSceneView* View, const FTexture* Texture, float GammaValue, ESimpleElementBlendMode BlendMode)
{
	SetEditorCompositingParameters(BatchedParameters, View);
	SetParameters(BatchedParameters, Texture, GammaValue, BlendMode);
}

FSimpleElementMaskedGammaBasePS::FSimpleElementMaskedGammaBasePS(const ShaderMetaType::CompiledShaderInitializerType& Initializer) :
	FSimpleElementGammaBasePS(Initializer)
{
	ClipRef.Bind(Initializer.ParameterMap,TEXT("ClipRef"), SPF_Mandatory);
}

void FSimpleElementMaskedGammaBasePS::SetParameters(FRHIBatchedShaderParameters& BatchedParameters, const FTexture* Texture, float InGamma, float ClipRefValue, ESimpleElementBlendMode BlendMode)
{
	FSimpleElementGammaBasePS::SetParameters(BatchedParameters, Texture, InGamma, BlendMode);
	SetShaderValue(BatchedParameters, ClipRef, ClipRefValue);
}

void FSimpleElementMaskedGammaBasePS::SetParameters(FRHIBatchedShaderParameters& BatchedParameters, const FSceneView* View, const FTexture* Texture, float InGamma, float ClipRefValue, ESimpleElementBlendMode BlendMode)
{
	SetEditorCompositingParameters(BatchedParameters, View);
	SetParameters(BatchedParameters, Texture, InGamma, ClipRefValue, BlendMode);
}

/**
* Constructor
*
* @param Initializer - shader initialization container
*/
FSimpleElementDistanceFieldGammaPS::FSimpleElementDistanceFieldGammaPS(const ShaderMetaType::CompiledShaderInitializerType& Initializer)
	: FSimpleElementMaskedGammaBasePS(Initializer)
{
	SmoothWidth.Bind(Initializer.ParameterMap,TEXT("SmoothWidth"));
	EnableShadow.Bind(Initializer.ParameterMap,TEXT("EnableShadow"));
	ShadowDirection.Bind(Initializer.ParameterMap,TEXT("ShadowDirection"));
	ShadowColor.Bind(Initializer.ParameterMap,TEXT("ShadowColor"));
	ShadowSmoothWidth.Bind(Initializer.ParameterMap,TEXT("ShadowSmoothWidth"));
	EnableGlow.Bind(Initializer.ParameterMap,TEXT("EnableGlow"));
	GlowColor.Bind(Initializer.ParameterMap,TEXT("GlowColor"));
	GlowOuterRadius.Bind(Initializer.ParameterMap,TEXT("GlowOuterRadius"));
	GlowInnerRadius.Bind(Initializer.ParameterMap,TEXT("GlowInnerRadius"));
}

void FSimpleElementDistanceFieldGammaPS::SetParameters(
	FRHIBatchedShaderParameters& BatchedParameters,
	const FTexture* Texture,
	float InGamma,
	float InClipRef,
	float SmoothWidthValue,
	bool bEnableShadowValue,
	const FVector2D& ShadowDirectionValue,
	const FLinearColor& ShadowColorValue,
	float ShadowSmoothWidthValue,
	const FDepthFieldGlowInfo& GlowInfo,
	ESimpleElementBlendMode BlendMode
)
{
	// This shader does not use editor compositing
	SetEditorCompositingParameters(BatchedParameters, nullptr);

	FSimpleElementMaskedGammaBasePS::SetParameters(BatchedParameters, Texture, InGamma, InClipRef, BlendMode);
	SetShaderValue(BatchedParameters, SmoothWidth, SmoothWidthValue);
	const uint32 bEnableShadowValueUInt = (bEnableShadowValue ? 1 : 0);
	SetShaderValue(BatchedParameters, EnableShadow, bEnableShadowValueUInt);
	if (bEnableShadowValue)
	{
		SetShaderValue(BatchedParameters, ShadowDirection, FVector2f(ShadowDirectionValue));
		SetShaderValue(BatchedParameters, ShadowColor, ShadowColorValue);
		SetShaderValue(BatchedParameters, ShadowSmoothWidth, ShadowSmoothWidthValue);
	}
	SetShaderValue(BatchedParameters, EnableGlow, GlowInfo.bEnableGlow);
	if (GlowInfo.bEnableGlow)
	{
		SetShaderValue(BatchedParameters, GlowColor, GlowInfo.GlowColor);
		SetShaderValue(BatchedParameters, GlowOuterRadius, FVector2f(GlowInfo.GlowOuterRadius));
		SetShaderValue(BatchedParameters, GlowInnerRadius, FVector2f(GlowInfo.GlowInnerRadius));
	}
}

/**
* Serialize constant paramaters for this shader
* 
* @param Ar - archive to serialize to
* @return true if any of the parameters were outdated
*/
/*bool FSimpleElementDistanceFieldGammaPS::Serialize(FArchive& Ar)
{
	bool bShaderHasOutdatedParameters = FSimpleElementMaskedGammaBasePS::Serialize(Ar);
	Ar << SmoothWidth;
	Ar << EnableShadow;
	Ar << ShadowDirection;
	Ar << ShadowColor;	
	Ar << ShadowSmoothWidth;
	Ar << EnableGlow;
	Ar << GlowColor;
	Ar << GlowOuterRadius;
	Ar << GlowInnerRadius;
	return bShaderHasOutdatedParameters;
}*/
	
FSimpleElementHitProxyPS::FSimpleElementHitProxyPS(const ShaderMetaType::CompiledShaderInitializerType& Initializer):
	FGlobalShader(Initializer)
{
	InTexture.Bind(Initializer.ParameterMap,TEXT("InTexture"), SPF_Mandatory);
	InTextureSampler.Bind(Initializer.ParameterMap,TEXT("InTextureSampler"));
}

bool FSimpleElementHitProxyPS::ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
{
	return IsPCPlatform(Parameters.Platform);
}

void FSimpleElementHitProxyPS::SetParameters(FRHIBatchedShaderParameters& BatchedParameters, const FTexture* TextureValue)
{
	SetTextureParameter(BatchedParameters, InTexture, InTextureSampler, TextureValue);
}

FSimpleElementColorChannelMaskPS::FSimpleElementColorChannelMaskPS(const ShaderMetaType::CompiledShaderInitializerType& Initializer)
: FGlobalShader(Initializer)
{
	InTexture.Bind(Initializer.ParameterMap,TEXT("InTexture"), SPF_Mandatory);
	InTextureSampler.Bind(Initializer.ParameterMap,TEXT("InTextureSampler"));
	ColorWeights.Bind(Initializer.ParameterMap,TEXT("ColorWeights"));
	Gamma.Bind(Initializer.ParameterMap,TEXT("Gamma"));
}

bool FSimpleElementColorChannelMaskPS::ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
{
	return IsPCPlatform(Parameters.Platform);
}

void FSimpleElementColorChannelMaskPS::SetParameters(FRHIBatchedShaderParameters& BatchedParameters, const FTexture* TextureValue, const FMatrix& ColorWeightsValue, float GammaValue)
{
	SetTextureParameter(BatchedParameters, InTexture, InTextureSampler, TextureValue);
	SetShaderValue(BatchedParameters, ColorWeights, (FMatrix44f)ColorWeightsValue);
	SetShaderValue(BatchedParameters, Gamma, GammaValue);
}

/*bool FSimpleElementColorChannelMaskPS::Serialize(FArchive& Ar)
{
	bool bShaderHasOutdatedParameters = FGlobalShader::Serialize(Ar);
	Ar << InTexture;
	Ar << InTextureSampler;
	Ar << ColorWeights;
	Ar << Gamma;
	return bShaderHasOutdatedParameters;
}*/

/*------------------------------------------------------------------------------
	Shader implementations.
------------------------------------------------------------------------------*/

IMPLEMENT_TYPE_LAYOUT(FSimpleElementGammaBasePS);
IMPLEMENT_TYPE_LAYOUT(FSimpleElementMaskedGammaBasePS);

IMPLEMENT_SHADER_TYPE(,FSimpleElementVS,TEXT("/Engine/Private/SimpleElementVertexShader.usf"),TEXT("Main"),SF_Vertex);
IMPLEMENT_SHADER_TYPE(,FSimpleElementPS, TEXT("/Engine/Private/SimpleElementPixelShader.usf"), TEXT("Main"), SF_Pixel);
IMPLEMENT_SHADER_TYPE(,FSimpleElementAlphaOnlyPS, TEXT("/Engine/Private/SimpleElementPixelShader.usf"), TEXT("AlphaOnlyMain"), SF_Pixel);
IMPLEMENT_SHADER_TYPE(template<>, FSimpleElementGammaPS_SRGB, TEXT("/Engine/Private/SimpleElementPixelShader.usf"), TEXT("GammaMain"), SF_Pixel);
IMPLEMENT_SHADER_TYPE(template<>, FSimpleElementGammaPS_Linear, TEXT("/Engine/Private/SimpleElementPixelShader.usf"), TEXT("GammaMain"), SF_Pixel);
IMPLEMENT_SHADER_TYPE(, FSimpleElementGammaAlphaOnlyPS, TEXT("/Engine/Private/SimpleElementPixelShader.usf"), TEXT("GammaAlphaOnlyMain"), SF_Pixel);
IMPLEMENT_SHADER_TYPE(template<>, FSimpleElementMaskedGammaPS_SRGB, TEXT("/Engine/Private/SimpleElementPixelShader.usf"), TEXT("GammaMaskedMain"), SF_Pixel);
IMPLEMENT_SHADER_TYPE(template<>, FSimpleElementMaskedGammaPS_Linear, TEXT("/Engine/Private/SimpleElementPixelShader.usf"), TEXT("GammaMaskedMain"), SF_Pixel);
IMPLEMENT_SHADER_TYPE(,FSimpleElementDistanceFieldGammaPS,TEXT("/Engine/Private/SimpleElementPixelShader.usf"),TEXT("GammaDistanceFieldMain"),SF_Pixel);
IMPLEMENT_SHADER_TYPE(,FSimpleElementHitProxyPS,TEXT("/Engine/Private/SimpleElementHitProxyPixelShader.usf"),TEXT("Main"),SF_Pixel);
IMPLEMENT_SHADER_TYPE(,FSimpleElementColorChannelMaskPS,TEXT("/Engine/Private/SimpleElementColorChannelMaskPixelShader.usf"),TEXT("Main"),SF_Pixel);

#undef IMPLEMENT_ENCODEDSHADERS
#undef BLEND_VARIATION
