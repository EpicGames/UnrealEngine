// Copyright Epic Games, Inc. All Rights Reserved.

#include "SingleAnimationPlayData.h"
#include "Animation/AnimSequenceBase.h"
#include "Animation/AnimSingleNodeInstance.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SingleAnimationPlayData)


void FSingleAnimationPlayData::Initialize(UAnimSingleNodeInstance* Instance)
{
	Instance->SetAnimationAsset(AnimToPlay);
	Instance->SetPosition(SavedPosition, false);
	Instance->SetPlayRate(SavedPlayRate);
	Instance->SetPlaying(bSavedPlaying);
	Instance->SetLooping(bSavedLooping);
}

void FSingleAnimationPlayData::PopulateFrom(UAnimSingleNodeInstance* Instance)
{
	AnimToPlay = Instance->GetCurrentAsset();
	SavedPosition = Instance->GetCurrentTime();
	SavedPlayRate = Instance->GetPlayRate();
	bSavedPlaying = Instance->IsPlaying();
	bSavedLooping = Instance->IsLooping();
}

void FSingleAnimationPlayData::ValidatePosition()
{
	float Min = 0, Max = 0;

	if (AnimToPlay)
	{
		UAnimSequenceBase* SequenceBase = Cast<UAnimSequenceBase>(AnimToPlay);
		if (SequenceBase)
		{
			Max = SequenceBase->GetPlayLength();
		}
	}

	SavedPosition = FMath::Clamp<float>(SavedPosition, Min, Max);
}

