// Copyright Epic Games, Inc. All Rights Reserved.

#include "SkeletalDebugRendering.h"
#include "HAL/IConsoleManager.h"
#include "DrawDebugHelpers.h"
#include "Math/RotationMatrix.h"
#include "MeshElementCollector.h"
#include "PrimitiveDrawingUtils.h"
#include "Engine/PoseWatchRenderData.h"
#include "ReferenceSkeleton.h"

static TAutoConsoleVariable<bool> CVarDisablePoseWatchRendering(
	TEXT("a.DisablePoseWatchRendering"),
	false,
	TEXT("Disable all active pose watches from being drawn."));

namespace SkeletalDebugRendering
{

/** A fast and simple bone drawing function. This draws a sphere and a pyramid connection to the PARENT bone.
 * Use this for basic debug drawing, but if the user is able to select or edit the bones, prefer DrawWireBoneAdvanced.*/
void DrawWireBone(
	FPrimitiveDrawInterface* PDI,
	const FVector& InStart,
	const FVector& InEnd,
	const FLinearColor& InColor,
	ESceneDepthPriorityGroup InDepthPriority,
	const float SphereRadius)
{
#if ENABLE_DRAW_DEBUG
	// Calc cone size 
	const FVector EndToStart = (InStart - InEnd);
	const float ConeLength = EndToStart.Size();
	const float Angle = FMath::RadiansToDegrees(FMath::Atan(SphereRadius / ConeLength));

	// Render Sphere for bone end point and a cone between it and its parent.
	DrawWireSphere(PDI, InEnd, InColor, SphereRadius, NumSphereSides, InDepthPriority, 0.0f, 1.0f);

	TArray<FVector> Verts;
	DrawWireCone(
		PDI,
		Verts,
		FRotationMatrix::MakeFromX(EndToStart) * FTranslationMatrix(InEnd),
		ConeLength,
		Angle,
		NumConeSides,
		InColor,
		InDepthPriority,
		0.0f,
		1.0f);
#endif
}

/** An advanced bone drawing function for use with interactive editors where the user can select and manipulate bones.
 *
 * Differences from DrawWireBone() include:
 * 1. Drawing all cone-connections to children as part of the "bone" itself so that the user can select the bone
 *	  by clicking on any of it's children connections (as in all DCC applications)
 * 2. Cone-connectors are drawn *between* spheres, not overlapping them (cleaner)
 * 3. Bone sphere is oriented with bone rotation.
 * 4. Connections to children can be colored individually to allow highlighting parent connections on selected children.
 *
 * This function, and the code required to structure the drawing in this manner, will incur some additional cost over
 * DrawWireBone(). So in cases where you just want to debug draw a skeleton; with no option to select or manipulate
 * the bones, it may be preferable to use DrawWireBone().
 */
void DrawWireBoneAdvanced(
	FPrimitiveDrawInterface* PDI,
	const FTransform& InBoneTransform,
	const TArray<FVector>& InChildLocations,
	const TArray<FLinearColor>& InChildColors,
	const FLinearColor& InColor,
	ESceneDepthPriorityGroup InDepthPriority,
	const float SphereRadius,
	const FBoneAxisDrawConfig& InAxisConfig)
{
#if ENABLE_DRAW_DEBUG

	const FVector BoneLocation = InBoneTransform.GetLocation();
	FTransform BoneNoScale = InBoneTransform;
	BoneNoScale.SetScale3D(FVector::OneVector);

	// draw wire sphere at joint origin, oriented with the bone
	DrawWireSphere(PDI, BoneNoScale, InColor, SphereRadius, NumSphereSides, InDepthPriority, 0.0f, 1.0f);

	// draw axes at joint location
	if (InAxisConfig.bDraw)
	{
		const float Thickness = InAxisConfig.Thickness > 0.f ? InAxisConfig.Thickness : 0.f;
		const float Length = InAxisConfig.Length > 0.f ? InAxisConfig.Length : SphereRadius;
		SkeletalDebugRendering::DrawAxes(PDI, BoneNoScale, SDPG_Foreground, Thickness, Length);
	}

	// draw wire cones to each child
	for (int32 ChildIndex=0; ChildIndex<InChildLocations.Num(); ++ChildIndex)
	{
		const FVector& ChildPoint = InChildLocations[ChildIndex];
		// offset start/end based on bone radius
		const FVector RadiusOffset = (ChildPoint - BoneLocation).GetSafeNormal() * SphereRadius;
		const FVector Start = BoneLocation + RadiusOffset;
		const FVector End = ChildPoint - RadiusOffset;
			
		// calc cone size
		const FVector EndToStart = (Start - End);
		const float ConeLength = EndToStart.Size();
		const float Angle = FMath::RadiansToDegrees(FMath::Atan(SphereRadius / ConeLength));
		TArray<FVector> Verts;
		DrawWireCone(
			PDI,
			Verts,
			FRotationMatrix::MakeFromX(EndToStart) * FTranslationMatrix(End),
			ConeLength,
			Angle,
			NumConeSides,
			InChildColors[ChildIndex],
			InDepthPriority,
			0.0f,
			1.0f);
	}
#endif
}

void DrawAxes(
	FPrimitiveDrawInterface* PDI,
	const FTransform& Transform,
	ESceneDepthPriorityGroup InDepthPriority,
	const float Thickness,
	const float AxisLength)
{
#if ENABLE_DRAW_DEBUG
	// Display colored coordinate system axes for this joint.
	const FVector Origin = Transform.GetLocation();

	// Red = X
	FVector XAxis = Transform.TransformVector(FVector(1.0f, 0.0f, 0.0f));
	XAxis.Normalize();
	PDI->DrawLine(Origin, Origin + XAxis * AxisLength, FColor(255, 80, 80), InDepthPriority, Thickness, 1.0f);

	// Green = Y
	FVector YAxis = Transform.TransformVector(FVector(0.0f, 1.0f, 0.0f));
	YAxis.Normalize();
	PDI->DrawLine(Origin, Origin + YAxis * AxisLength, FColor(80, 255, 80), InDepthPriority, Thickness, 1.0f);

	// Blue = Z
	FVector ZAxis = Transform.TransformVector(FVector(0.0f, 0.0f, 1.0f));
	ZAxis.Normalize();
	PDI->DrawLine(Origin, Origin + ZAxis * AxisLength, FColor(80, 80, 255), InDepthPriority, Thickness, 1.0f);
#endif
}

void DrawConeConnection(
	FPrimitiveDrawInterface* PDI,
	const FVector& Start,
	const FVector& End,
	const float SphereRadius,
	const FLinearColor& Color)
{
#if ENABLE_DRAW_DEBUG
	// offset start/end based on bone radius
	const FVector RadiusOffset = (End - Start).GetSafeNormal() * SphereRadius;
	const FVector StartOffset = Start + RadiusOffset;
			
	// calc cone size
	const FVector EndToStart = (StartOffset - End);
	const float ConeLength = EndToStart.Size();
	const float Angle = FMath::RadiansToDegrees(FMath::Atan(SphereRadius / ConeLength));
	TArray<FVector> Verts;
	DrawWireCone(
		PDI,
		Verts,
		FRotationMatrix::MakeFromX(EndToStart) * FTranslationMatrix(End),
		ConeLength,
		Angle,
		NumConeSides,
		Color,
		ESceneDepthPriorityGroup::SDPG_Foreground,
		0.0f,
		1.0f);
#endif
}

#if WITH_EDITOR
void DrawBonesFromPoseWatch(
	FPrimitiveDrawInterface* PDI,
	const FAnimNodePoseWatch& PoseWatch,
	const bool bUseWorldTransform
)
{
	if (CVarDisablePoseWatchRendering.GetValueOnAnyThread())
	{
		return;
	}

	const TArray<FTransform>& InBoneTransforms = PoseWatch.GetBoneTransforms();
	const TArray<FBoneIndexType>& InRequiredBones = PoseWatch.GetRequiredBones();
	if (InRequiredBones.Num() == 0 || InBoneTransforms.Num() < InRequiredBones.Num())
	{
		return;
	}

	const FTransform WorldTransform = bUseWorldTransform ? PoseWatch.GetWorldTransform() : FTransform::Identity;
	const FVector RelativeOffset = WorldTransform.GetRotation().RotateVector(PoseWatch.GetViewportOffset());

	const TArray<int32>& ViewportMaskAllowList = PoseWatch.GetViewportAllowList();
	const TArray<int32>& ParentIndices = PoseWatch.GetParentIndices();

	TArray<FTransform> UseWorldTransforms;
	UseWorldTransforms.AddDefaulted(InBoneTransforms.Num());
	
	TArray<FBoneIndexType> UseRequiredBones;
	UseRequiredBones.Reserve(InRequiredBones.Num());

	for (const FBoneIndexType& BoneIndex : InRequiredBones)
	{
		if (ParentIndices.IsValidIndex(BoneIndex) && InBoneTransforms.IsValidIndex(BoneIndex))
		{
			const int32 ParentIndex = ParentIndices[BoneIndex];

			if (ParentIndex == INDEX_NONE)
			{
				UseWorldTransforms[BoneIndex] = InBoneTransforms[BoneIndex] * WorldTransform;
				UseWorldTransforms[BoneIndex].AddToTranslation(RelativeOffset);
			}
			else
			{
				UseWorldTransforms[BoneIndex] = InBoneTransforms[BoneIndex] * UseWorldTransforms[ParentIndex];
			}

			if (!ViewportMaskAllowList.Contains(BoneIndex))
			{
				continue;
			}

			UseRequiredBones.Add(BoneIndex);
		}
	}

	const FLinearColor BoneColor = PoseWatch.GetBoneColor();

	FSkelDebugDrawConfig DrawConfig;
	DrawConfig.BoneDrawMode = EBoneDrawMode::All;
	DrawConfig.BoneDrawSize = 1.f;
	DrawConfig.bAddHitProxy = false;
	DrawConfig.bForceDraw = true;
	DrawConfig.DefaultBoneColor = BoneColor;
	DrawConfig.AffectedBoneColor = BoneColor;
	DrawConfig.SelectedBoneColor = BoneColor;
	DrawConfig.ParentOfSelectedBoneColor = BoneColor;
	DrawConfig.bUseMultiColorAsDefaultColor = false;

	SkeletalDebugRendering::DrawBonesInternal(
		PDI,
		WorldTransform.GetLocation() + RelativeOffset,
		UseRequiredBones,
		ParentIndices,
		UseWorldTransforms,
		/*SelectedBones*/TArray<int32>(),
		/*BoneColors*/TArray<FLinearColor>(),
		/*HitProxies*/TArray<TRefCountPtr<HHitProxy>>(),
		DrawConfig,
		TBitArray<>{});
}
#endif

void DrawBones(
	FPrimitiveDrawInterface* PDI,
	const FVector& ComponentOrigin,
	const TArray<FBoneIndexType>& RequiredBones,
	const FReferenceSkeleton& RefSkeleton,
	const TArray<FTransform>& WorldTransforms,
	const TArray<int32>& InSelectedBones,
	const TArray<FLinearColor>& BoneColors,
	const TArray<TRefCountPtr<HHitProxy>>& HitProxies,
	const FSkelDebugDrawConfig& DrawConfig)
{
	DrawBones(
		PDI,
		ComponentOrigin,
		RequiredBones,
		RefSkeleton,
		WorldTransforms,
		InSelectedBones,
		BoneColors,
		HitProxies,
		DrawConfig,
		TBitArray<>{});
}

void DrawBones(
	FPrimitiveDrawInterface* PDI,
	const FVector& ComponentOrigin,
	const TArray<FBoneIndexType>& RequiredBones,
	const FReferenceSkeleton& RefSkeleton,
	const TArray<FTransform>& WorldTransforms,
	const TArray<int32>& InSelectedBones,
	const TArray<FLinearColor>& BoneColors,
	const TArray<TRefCountPtr<HHitProxy>>& HitProxies,
	const FSkelDebugDrawConfig& DrawConfig,
	// Overrides the bones that're drawn
	const TBitArray<>& BonesToDrawOverride)
{
	// get parent indices of bones
	TArray<int32> ParentIndices;
	ParentIndices.AddUninitialized(RefSkeleton.GetNum());
	for (int32 BoneIndex = 0; BoneIndex < RefSkeleton.GetNum(); ++BoneIndex)
	{
		ParentIndices[BoneIndex] = RefSkeleton.GetParentIndex(BoneIndex);
	}

	SkeletalDebugRendering::DrawBonesInternal(
		PDI,
		ComponentOrigin,
		RequiredBones,
		ParentIndices,
		WorldTransforms,
		InSelectedBones,
		BoneColors,
		HitProxies,
		DrawConfig,
		BonesToDrawOverride);
}


void DrawBonesInternal(
	FPrimitiveDrawInterface* PDI,
	const FVector& ComponentOrigin,
	const TArray<FBoneIndexType>& RequiredBones,
	const TArray<int32>& ParentIndices,
	const TArray<FTransform>& WorldTransforms,
	const TArray<int32>& InSelectedBones,
	const TArray<FLinearColor>& BoneColors,
	const TArray<TRefCountPtr<HHitProxy>>& HitProxies,
	const FSkelDebugDrawConfig& DrawConfig,
	const TBitArray<>& BonesToDrawOverride)
{
	const auto GetParentIndex = [ParentIndices](const int32 InBoneIndex) -> int32
	{
		if (ParentIndices.IsValidIndex(InBoneIndex))
		{
			return ParentIndices[InBoneIndex];
		}
		return INDEX_NONE;
	};

	// first determine which bones to draw, and which to filter out
	const int32 NumBones = ParentIndices.Num();
	const bool bDrawAll = DrawConfig.BoneDrawMode == EBoneDrawMode::All;
	const bool bDrawSelected = DrawConfig.BoneDrawMode == EBoneDrawMode::Selected;
	const bool bDrawSelectedAndParents = DrawConfig.BoneDrawMode == EBoneDrawMode::SelectedAndParents;
	const bool bDrawSelectedAndChildren = DrawConfig.BoneDrawMode == EBoneDrawMode::SelectedAndChildren;
	const bool bDrawSelectedAndParentsAndChildren = DrawConfig.BoneDrawMode == EBoneDrawMode::SelectedAndParentsAndChildren;

	TBitArray<> BonesToDraw = BonesToDrawOverride;
	if (BonesToDraw.IsEmpty())
	{
		CalculateBonesToDraw(ParentIndices, InSelectedBones, DrawConfig.BoneDrawMode, BonesToDraw);
	}

	// determine which bones are "affected" (these are ALL children of selected bones)
	TBitArray<> AffectedBones(false, NumBones);
	for (int32 BoneIndex = 0; BoneIndex < NumBones; ++BoneIndex)
	{
		for (int32 ParentIndex = GetParentIndex(BoneIndex); ParentIndex != INDEX_NONE; ParentIndex = GetParentIndex(ParentIndex))
		{
			if (InSelectedBones.Contains(ParentIndex))
			{
				AffectedBones[BoneIndex] = true;
				break;
			}
		}
	}

	FBoneAxisDrawConfig AxisConfig = DrawConfig.AxisConfig;
	
	// spin through all required bones and render them
	const float BoneRadius = DrawConfig.BoneDrawSize;
	for (int32 Index = 0; Index < RequiredBones.Num(); ++Index)
	{
		const int32 BoneIndex = RequiredBones[Index];
		if ((!BoneColors.IsEmpty() && BoneIndex >= BoneColors.Num()) ||
			BoneIndex >= WorldTransforms.Num() ||
			!BonesToDraw.IsValidIndex(BoneIndex))
		{
			continue;
		}

		// skips bones that should not be drawn
		const bool bDoDraw = DrawConfig.bForceDraw || BonesToDraw[BoneIndex];
		if (!bDoDraw)
		{
			continue;
		}

		// determine color of bone based on selection / affected state
		const bool bIsSelected = InSelectedBones.Contains(BoneIndex);
		const bool bIsAffected = AffectedBones[BoneIndex];
		FLinearColor DefaultBoneColor;
		if (BoneColors.IsEmpty())
		{
			DefaultBoneColor = DrawConfig.bUseMultiColorAsDefaultColor ? GetSemiRandomColorForBone(BoneIndex) : DrawConfig.DefaultBoneColor;
		}
		else
		{
			DefaultBoneColor = BoneColors[BoneIndex];
		}
		FLinearColor BoneColor = bIsAffected ? DrawConfig.AffectedBoneColor : DefaultBoneColor;
		BoneColor = bIsSelected ? DrawConfig.SelectedBoneColor : BoneColor;

		// Draw the local rotation axes for affected and selected bones, or for all related option is set to true
		AxisConfig.bDraw = DrawConfig.bDrawAllLocalRotationAxes || bIsAffected || bIsSelected;

		// draw cone to each child
		// but use a different color if this bone is NOT selected, but the child IS selected
		TArray<FVector> ChildPositions;
		TArray<FLinearColor> ChildColors;
		for (int32 ChildIndex = 0; ChildIndex < NumBones; ++ChildIndex)
		{
			const int32 ParentIndex = GetParentIndex(ChildIndex);
			if (ParentIndex >= WorldTransforms.Num())
			{
				continue;
			}
			if (ParentIndex == BoneIndex && RequiredBones.Contains(ChildIndex))
			{
				ChildPositions.Add(WorldTransforms[ChildIndex].GetLocation());
				FLinearColor ChildLineColor = BoneColor;
				if (!bIsSelected && InSelectedBones.Contains(ChildIndex))
				{
					ChildLineColor = DrawConfig.ParentOfSelectedBoneColor;
				}
				ChildColors.Add(ChildLineColor);
			}
		}

		const FTransform BoneTransform = WorldTransforms[BoneIndex];

		// Always set new hit proxy to prevent unintentionally using last drawn element's proxy
		PDI->SetHitProxy(DrawConfig.bAddHitProxy ? HitProxies[BoneIndex] : nullptr);

		// draw skeleton
		SkeletalDebugRendering::DrawWireBoneAdvanced(
			PDI,
			BoneTransform,
			ChildPositions,
			ChildColors,
			BoneColor,
			SDPG_Foreground,
			BoneRadius,
			AxisConfig);
		
		// special case for root connection to origin
		if (GetParentIndex(BoneIndex) == INDEX_NONE)
		{
			SkeletalDebugRendering::DrawConeConnection(PDI, BoneTransform.GetLocation(), ComponentOrigin, BoneRadius, FLinearColor::Red);
		}
		
		// special case for forcing drawing connection to parent when:
		// 1. parent is not selected AND
		// 2. only drawing selected or children of selected bones
		// In this case, the connection to the parent will not get drawn unless we force it here
		if (bDrawSelected || bDrawSelectedAndChildren)
		{
			if (InSelectedBones.Contains(BoneIndex))
			{
				const int32 ParentIndex = GetParentIndex(BoneIndex);
				if (!WorldTransforms.IsValidIndex(ParentIndex))
				{
					continue;
				}
				const FVector ParentPosition = WorldTransforms[ParentIndex].GetTranslation();
				SkeletalDebugRendering::DrawConeConnection(PDI, ParentPosition, BoneTransform.GetLocation(), BoneRadius, DrawConfig.ParentOfSelectedBoneColor);
			}
		}
		
		PDI->SetHitProxy(nullptr);
	}
}

FLinearColor GetSemiRandomColorForBone(const int32 BoneIndex, float Value, float Saturation)
{
	// uses deterministic, semi-random desaturated color unique to the bone index
	constexpr float Rotation = 90.f;
	return FLinearColor::IntToDistinctColor(BoneIndex, Saturation, Value, Rotation);
}

void FillWithMultiColors(TArray<FLinearColor>& BoneColors, const int32 NumBones)
{
	BoneColors.SetNumUninitialized(NumBones);
	for (int32 BoneIndex=0; BoneIndex<NumBones; ++BoneIndex)
	{
		BoneColors[BoneIndex] = GetSemiRandomColorForBone(BoneIndex);
	}
}


void CalculateBonesToDraw(
	const TArray<int32>& ParentIndices,
	const TArray<int32>& InSelectedBones,
	const EBoneDrawMode::Type BoneDrawMode,
	TBitArray<>& OutBonesToDraw)
{
	const auto GetParentIndex = [ParentIndices](const int32 InBoneIndex) -> int32
	{
		if (ParentIndices.IsValidIndex(InBoneIndex))
		{
			return ParentIndices[InBoneIndex];
		}
		return INDEX_NONE;
	};

	const int32 NumBones = ParentIndices.Num();
	OutBonesToDraw.Init(false, NumBones);

	const bool bDrawAll = BoneDrawMode == EBoneDrawMode::All;
	const bool bDrawSelected = BoneDrawMode == EBoneDrawMode::Selected;
	const bool bDrawSelectedAndParents = BoneDrawMode == EBoneDrawMode::SelectedAndParents;
	const bool bDrawSelectedAndChildren = BoneDrawMode == EBoneDrawMode::SelectedAndChildren;
	const bool bDrawSelectedAndParentsAndChildren = BoneDrawMode == EBoneDrawMode::SelectedAndParentsAndChildren;

	// draw all bones
	if (bDrawAll)
	{
		OutBonesToDraw.Init(true, NumBones);
	}

	// add selected bones
	if (bDrawSelected || bDrawSelectedAndParents || bDrawSelectedAndChildren || bDrawSelectedAndParentsAndChildren)
	{
		for (int32 BoneIndex : InSelectedBones)
		{
			if (BoneIndex != INDEX_NONE && OutBonesToDraw.IsValidIndex(BoneIndex))
			{
				OutBonesToDraw[BoneIndex] = true;
			}
		}
	}

	// add children of selected
	if (bDrawSelectedAndChildren || bDrawSelectedAndParentsAndChildren)
	{
		for (int32 BoneIndex = 0; BoneIndex < NumBones; ++BoneIndex)
		{
			const int32 ParentIndex = GetParentIndex(BoneIndex);
			if (ParentIndex != INDEX_NONE && OutBonesToDraw[ParentIndex])
			{
				OutBonesToDraw[BoneIndex] = true;
			}
		}
	}

	// add parents of selected
	if (bDrawSelectedAndParents || bDrawSelectedAndParentsAndChildren)
	{
		for (const int32 BoneIndex : InSelectedBones)
		{
			if (BoneIndex != INDEX_NONE)
			{
				for (int32 ParentIndex = GetParentIndex(BoneIndex); ParentIndex != INDEX_NONE; ParentIndex = GetParentIndex(ParentIndex))
				{
					OutBonesToDraw[ParentIndex] = true;
				}
			}
		}
	}
}

}
