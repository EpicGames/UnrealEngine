// Copyright Epic Games, Inc. All Rights Reserved.

#include "BodySetupEnums.h"
#include "Misc/ConfigCacheIni.h"
#include "ClothingAssetBase.h"
#include "ClothingSimulationFactory.h"
#include "ClothingSimulationInstance.h"
#include "Misc/ConfigCacheIni.h"
#include "ClothCollisionSource.h"
#include "Components/InstancedStaticMeshComponent.h"
#include "SkeletalMeshSceneProxy.h"
#include "Engine/OverlapResult.h"
#include "Engine/SkeletalMesh.h"
#include "Engine/World.h"
#include "Modules/ModuleManager.h"
#include "Physics/Experimental/PhysScene_Chaos.h"
#include "PhysicsEngine/EnvironmentalCollisions.h"
#include "PhysicsProxy/SingleParticlePhysicsProxy.h"
#include "AnimationRuntime.h"
#include "ClothCollisionData.h"
#include "ClothingSimulationInteractor.h"
#include "ClothingSimulationTeleportHelpers.h"
#include "Rendering/RenderCommandPipes.h"

#include "Logging/MessageLog.h"
#include "CollisionDebugDrawingPublic.h"


#include "PhysicsEngine/BodySetup.h"
#include "PhysicsEngine/PhysicsConstraintTemplate.h"
#include "PhysicsEngine/PhysicsAsset.h"
#include "PhysicsEngine/SkeletalBodySetup.h"
#include "PhysicsEngine/SphylElem.h"
#include "PhysicsEngine/TaperedCapsuleElem.h"
#include "Rendering/SkeletalMeshLODRenderData.h"
#include "RenderingThread.h"
#include "SceneInterface.h"

#if WITH_EDITOR
#include "ClothingSystemEditorInterfaceModule.h"
#include "PhysicsEngine/SphereElem.h"
#include "SimulationEditorExtender.h"
#endif

#include "Chaos/Capsule.h"
#include "Chaos/Convex.h"
#include "Chaos/ImplicitFwd.h"
#include "Chaos/ImplicitObjectScaled.h"
#include "Chaos/PhysicsObjectInterface.h"
#include "Physics/PhysicsReplicationQuantization.h"

#define LOCTEXT_NAMESPACE "SkeletalMeshComponentPhysics"

DECLARE_CYCLE_STAT(TEXT("CreateClothing"), STAT_CreateClothing, STATGROUP_Physics);

CSV_DECLARE_CATEGORY_MODULE_EXTERN(ENGINE_API, Animation);
CSV_DECLARE_CATEGORY_MODULE_EXTERN(CORE_API, Basic);

TAutoConsoleVariable<int32> CVarEnableClothPhysics(TEXT("p.ClothPhysics"), 1, TEXT("If 1, physics cloth will be used for simulation."), ECVF_Scalability);
TAutoConsoleVariable<int32> CVarClothPhysicsLODThreshold(TEXT("p.ClothPhysics.LODThreshold"), -1, TEXT("Max LOD that cloth simulation is allowed to run on. Use -1 to disable."), ECVF_Scalability);

static TAutoConsoleVariable<int32> CVarEnableClothPhysicsUseTaskThread(TEXT("p.ClothPhysics.UseTaskThread"), 1, TEXT("If 1, run cloth on the task thread. If 0, run on game thread."));
static TAutoConsoleVariable<int32> CVarClothPhysicsTickWaitForParallelClothTask(TEXT("p.ClothPhysics.WaitForParallelClothTask"), 0, TEXT("If 1, always wait for cloth task completion in the Cloth Tick function. If 0, wait at end-of-frame updates instead if allowed by component settings"));
static TAutoConsoleVariable<int32> CVarClothPhysicsUseEndTickWait(TEXT("p.ClothPhysics.UseEndTickWait"), 1, TEXT("If 1 (default), wait on the parallel cloth/deformable task in a dedicated TG_LastDemotable tick function. If 0, fall back to waiting in OnPreEndOfFrameSync (kill switch). Has no effect when p.ClothPhysics.WaitForParallelClothTask is 1."));
static TAutoConsoleVariable<int32> CVarClothPhysicsTickWithMontages(TEXT("p.ClothPhysics.TickWithMontages"), 0, TEXT("Set to 1 to keep legacy behavior and tick clothing with montage related VisibilityBasedAnimTickOption."));
static TAutoConsoleVariable<bool> CVarClothPhysicsOnUpdateTransformTeleport(TEXT("p.ClothPhysics.OnUpdateTransformTeleport"), true, TEXT("Allow transform changes to trigger cloth teleports."));

static TAutoConsoleVariable<int32> CVarEnableKinematicDeferralPrePhysicsCondition(TEXT("p.EnableKinematicDeferralPrePhysicsCondition"), 1, TEXT("If is 1, and deferral would've been disallowed due to EUpdateTransformFlags, allow if in PrePhysics tick. If 0, condition is unchanged."));

static TAutoConsoleVariable<bool> CVarUpdateRigidBodyScaling(TEXT("p.UpdateRigidBodyScaling"), true, TEXT("Updates the rigid body scaling for simulation when the actor scaling is changed \n Default: true."));

static bool GEnableSkeletalMeshComponentAsyncInitBody = false;
static FAutoConsoleVariableRef CVarEnableSkeletalMeshComponentAsyncInitBody(TEXT("p.SkeletalMeshComponent.EnableAsyncInitBody"), GEnableSkeletalMeshComponentAsyncInitBody, TEXT("Enables SkeletalMeshComponent's body instances to be initialized outside of game thread (default is false)."), ECVF_ReadOnly);

// Used as a default return value for invalid cloth data access
static const TMap<int32, FClothSimulData> SEmptyClothSimulationData;


void FSkeletalMeshComponentClothTickFunction::ExecuteTick(float DeltaTime, enum ELevelTick TickType, ENamedThreads::Type CurrentThread, const FGraphEventRef& MyCompletionGraphEvent)
{
	QUICK_SCOPE_CYCLE_COUNTER(FSkeletalMeshComponentClothTickFunction_ExecuteTick);
#if WITH_EDITOR
	FActorComponentTickFunction::ExecuteTickHelper(Target, Target->bUpdateClothInEditor, DeltaTime, TickType, [this](float DilatedTime)
#else
	FActorComponentTickFunction::ExecuteTickHelper(Target,true, DeltaTime, TickType, [this](float DilatedTime)
#endif
	{
		Target->TickClothing(DilatedTime, *this);
	});
}

FString FSkeletalMeshComponentClothTickFunction::DiagnosticMessage()
{
	if (Target)
	{
		return Target->GetFullName() + TEXT("[ClothTick]");
	}
	return TEXT("<NULL>[ClothTick]");
}

FName FSkeletalMeshComponentClothTickFunction::DiagnosticContext(bool bDetailed)
{
	return FName("SkeletalMeshComponentClothTick");
}

void FSkeletalMeshComponentClothEndTickFunction::ExecuteTick(float DeltaTime, enum ELevelTick TickType, ENamedThreads::Type CurrentThread, const FGraphEventRef& MyCompletionGraphEvent)
{
	QUICK_SCOPE_CYCLE_COUNTER(FSkeletalMeshComponentClothEndTickFunction_ExecuteTick);
	if (Target)
	{
		Target->HandleExistingParallelClothSimulation();
	}
}

FString FSkeletalMeshComponentClothEndTickFunction::DiagnosticMessage()
{
	if (Target)
	{
		return Target->GetFullName() + TEXT("[ClothEndTick]");
	}
	return TEXT("<NULL>[ClothEndTick]");
}

FName FSkeletalMeshComponentClothEndTickFunction::DiagnosticContext(bool bDetailed)
{
	return FName("SkeletalMeshComponentClothEndTick");
}

void FSkeletalMeshComponentEndPhysicsTickFunction::ExecuteTick(float DeltaTime, enum ELevelTick TickType, ENamedThreads::Type CurrentThread, const FGraphEventRef& MyCompletionGraphEvent)
{
	QUICK_SCOPE_CYCLE_COUNTER(FSkeletalMeshComponentEndPhysicsTickFunction_ExecuteTick);
	CSV_SCOPED_TIMING_STAT_EXCLUSIVE(Physics);

	FActorComponentTickFunction::ExecuteTickHelper(Target, /*bTickInEditor=*/ false, DeltaTime, TickType, [this](float DilatedTime)
	{
		Target->EndPhysicsTickComponent(*this);
	});
}

FString FSkeletalMeshComponentEndPhysicsTickFunction::DiagnosticMessage()
{
	if (Target)
	{
		return Target->GetFullName() + TEXT("[EndPhysicsTick]");
	}
	return TEXT("<NULL>[EndPhysicsTick]");
}

FName FSkeletalMeshComponentEndPhysicsTickFunction::DiagnosticContext(bool bDetailed)
{
	return FName("SkeletalMeshComponentEndPhysicsTick");
}

FClothCollisionSource::FClothCollisionSource(USkeletalMeshComponent* InSourceComponent, UPhysicsAsset* InSourcePhysicsAsset, const FOnBoneTransformsFinalizedMultiCast::FDelegate& InOnBoneTransformsFinalizedDelegate)
	: SourceComponent(InSourceComponent)
	, SourcePhysicsAsset(InSourcePhysicsAsset)
	, bCached(false)
{
	if (SourceComponent.IsValid())
	{
		OnBoneTransformsFinalizedHandle = InSourceComponent->RegisterOnBoneTransformsFinalizedDelegate(InOnBoneTransformsFinalizedDelegate);
	}
}

FClothCollisionSource::~FClothCollisionSource()
{
	if (SourceComponent.IsValid() && OnBoneTransformsFinalizedHandle.IsValid())
	{
		SourceComponent->UnregisterOnBoneTransformsFinalizedDelegate(OnBoneTransformsFinalizedHandle);
	}
}

void FClothCollisionSource::ExtractCollisions(const USkeletalMeshComponent* DestClothComponent, FClothCollisionData& OutCollisions)
{
	const USkeletalMeshComponent* const SourceComponentRawPtr = SourceComponent.Get();
	const UPhysicsAsset* const SourcePhysicsAssetRawPtr = SourcePhysicsAsset.Get();

	// Extract collisions from this mesh 'raw', as this isn't a mesh that has cloth simulation
	// (but we want it to affect other meshes with cloth simulation)
	if (SourceComponentRawPtr && SourceComponentRawPtr->GetSkeletalMeshAsset() && SourcePhysicsAssetRawPtr)
	{
		FTransform ComponentToComponentTransform;
		if (SourceComponentRawPtr != DestClothComponent)
		{
			FTransform DestClothComponentTransform = DestClothComponent->GetComponentTransform();
			DestClothComponentTransform.RemoveScaling();  // The collision source doesn't need the scale of the cloth skeletal mesh applied to it (but it does need the source scale from the component transform)
			ComponentToComponentTransform = SourceComponentRawPtr->GetComponentTransform() * DestClothComponentTransform.Inverse();
		}

		// Init cache on first copy
		if (!bCached || CachedSkeletalMesh.Get() != SourceComponentRawPtr->GetSkeletalMeshAsset())
		{
			// Clear previous cached data
			CachedSpheres.Reset();
			CachedSphereConnections.Reset();

			for (const USkeletalBodySetup* SkeletalBodySetup : SourcePhysicsAssetRawPtr->SkeletalBodySetups)
			{
				// Cache bones
				const int32 MeshBoneIndex = SourceComponentRawPtr->GetSkeletalMeshAsset()->GetRefSkeleton().FindBoneIndex(SkeletalBodySetup->BoneName);
				if (MeshBoneIndex != INDEX_NONE)
				{
					// Cache spheres & capsules form physics asset
					for (const FKSphereElem& Sphere : SkeletalBodySetup->AggGeom.SphereElems)
					{
						FClothCollisionPrim_Sphere NewSphere;
						NewSphere.LocalPosition = Sphere.Center;
						NewSphere.Radius = Sphere.Radius;
						NewSphere.BoneIndex = MeshBoneIndex;

						CachedSpheres.Add(NewSphere);
					}

					for (const FKSphylElem& Sphyl : SkeletalBodySetup->AggGeom.SphylElems)
					{
						FClothCollisionPrim_Sphere Sphere0;
						FClothCollisionPrim_Sphere Sphere1;
						FVector OrientedDirection = Sphyl.Rotation.RotateVector(FVector(0.0f, 0.0f, 1.0f));
						FVector HalfDim = OrientedDirection * (Sphyl.Length / 2.0f);
						Sphere0.LocalPosition = Sphyl.Center - HalfDim;
						Sphere1.LocalPosition = Sphyl.Center + HalfDim;
						Sphere0.Radius = Sphyl.Radius;
						Sphere1.Radius = Sphyl.Radius;
						Sphere0.BoneIndex = MeshBoneIndex;
						Sphere1.BoneIndex = MeshBoneIndex;

						CachedSpheres.Add(Sphere0);
						CachedSpheres.Add(Sphere1);

						FClothCollisionPrim_SphereConnection Connection;
						Connection.SphereIndices[0] = CachedSpheres.Num() - 2;
						Connection.SphereIndices[1] = CachedSpheres.Num() - 1;

						CachedSphereConnections.Add(Connection);
					}

					for (const FKTaperedCapsuleElem& TaperedCapsuleElems : SkeletalBodySetup->AggGeom.TaperedCapsuleElems)
					{
						FClothCollisionPrim_Sphere Sphere0;
						FClothCollisionPrim_Sphere Sphere1;
						const FVector OrientedDirection = TaperedCapsuleElems.Rotation.RotateVector(FVector(0.0f, 0.0f, 1.0f));
						const FVector HalfDim = OrientedDirection * (TaperedCapsuleElems.Length / 2.0f);
						Sphere0.LocalPosition = TaperedCapsuleElems.Center + HalfDim;
						Sphere1.LocalPosition = TaperedCapsuleElems.Center - HalfDim;
						Sphere0.Radius = TaperedCapsuleElems.Radius0;
						Sphere1.Radius = TaperedCapsuleElems.Radius1;
						Sphere0.BoneIndex = MeshBoneIndex;
						Sphere1.BoneIndex = MeshBoneIndex;

						CachedSpheres.Add(Sphere0);
						CachedSpheres.Add(Sphere1);

						FClothCollisionPrim_SphereConnection Connection;
						Connection.SphereIndices[0] = CachedSpheres.Num() - 2;
						Connection.SphereIndices[1] = CachedSpheres.Num() - 1;
						if (TaperedCapsuleElems.bOneSidedCollision)
						{
							Connection.OneSidedPlaneNormal = TaperedCapsuleElems.Rotation.RotateVector(FVector::ForwardVector);
						}

						CachedSphereConnections.Add(Connection);
					}
				}
			}
			CachedSkeletalMesh = SourceComponentRawPtr->GetSkeletalMeshAsset();
			bCached = true;
		}

		// Presize array allocations
		OutCollisions.Spheres.Reserve(OutCollisions.Spheres.Num() + CachedSpheres.Num());
		OutCollisions.SphereConnections.Reserve(OutCollisions.SphereConnections.Num() + CachedSphereConnections.Num());

		// Now transform output data
		for (const FClothCollisionPrim_Sphere& CachedSphere : CachedSpheres)
		{
			FClothCollisionPrim_Sphere& OutSphere = OutCollisions.Spheres.Add_GetRef(CachedSphere);

			const FTransform BoneTransform = SourceComponentRawPtr->GetBoneTransform(OutSphere.BoneIndex, FTransform::Identity) * ComponentToComponentTransform;
			OutSphere.LocalPosition = BoneTransform.TransformPosition(OutSphere.LocalPosition);
			OutSphere.Radius *= BoneTransform.GetScale3D().X;  // Cloth collisions only uniformly scale
			OutSphere.BoneIndex = INDEX_NONE;
		}

		// Offset connections
		const int32 ConnectionBaseIndex = OutCollisions.SphereConnections.Num();
		for (const FClothCollisionPrim_SphereConnection& CachedSphereConnection : CachedSphereConnections)
		{
			FClothCollisionPrim_SphereConnection& OutSphereConnection = OutCollisions.SphereConnections.Add_GetRef(CachedSphereConnection);
			OutSphereConnection.SphereIndices[0] += ConnectionBaseIndex;
			OutSphereConnection.SphereIndices[1] += ConnectionBaseIndex;
		}
	}
}

void USkeletalMeshComponent::CreateBodySetup()
{
	if (BodySetup == NULL)
	{
		BodySetup = NewObject<UBodySetup>(this);
	}

	if (GetSkeletalMeshAsset())
	{
		const USkeletalMesh* SkeletalMeshConst = GetSkeletalMeshAsset();
		GetSkeletalMeshAsset()->CreateBodySetup();
		UBodySetup* OriginalBodySetup = SkeletalMeshConst->GetBodySetup();
		BodySetup->CopyBodyPropertiesFrom(OriginalBodySetup);
		BodySetup->CookedFormatProvider = SkeletalMeshConst;
		BodySetup->bHasCookedFormatProvider = true;
	}

	BodySetup->CollisionTraceFlag = CTF_UseComplexAsSimple;

	//need to recreate meshes
	BodySetup->ClearPhysicsMeshes();
	BodySetup->CreatePhysicsMeshes();
}

//
//	USkeletalMeshComponent
//
UBodySetup* USkeletalMeshComponent::GetBodySetup()
{
	if (bEnablePerPolyCollision == false)
	{
		UPhysicsAsset * const PhysicsAsset = GetPhysicsAsset();
		if (GetSkeletalMeshAsset() && PhysicsAsset)
		{
			for (int32 i = 0; i < GetSkeletalMeshAsset()->GetRefSkeleton().GetNum(); i++)
			{
				int32 BodyIndex = PhysicsAsset->FindBodyIndex(GetSkeletalMeshAsset()->GetRefSkeleton().GetBoneName(i));
				if (BodyIndex != INDEX_NONE)
				{
					return PhysicsAsset->SkeletalBodySetups[BodyIndex];
				}
			}
		}
	}
	else
	{
		if (BodySetup == NULL)
		{
			CreateBodySetup();
		}

		return BodySetup;
	}


	return NULL;
}

bool USkeletalMeshComponent::CanEditSimulatePhysics()
{
#if WITH_EDITOR
	return (GetSkinnedAsset() != nullptr) && !GetSkinnedAsset()->IsCompiling() && (GetPhysicsAsset() != nullptr);
#else
	return GetPhysicsAsset() != nullptr;
#endif
}

void USkeletalMeshComponent::SetSimulatePhysics(bool bSimulate)
{
	if ( !bEnablePhysicsOnDedicatedServer && IsRunningDedicatedServer() )
	{
		return;
	}

	BodyInstance.bSimulatePhysics = bSimulate;

	// enable blending physics
	bBlendPhysics = bSimulate;

	//Go through body setups and see which bodies should be turned on and off
	if (UPhysicsAsset * PhysAsset = GetPhysicsAsset())
	{
		for (int32 BodyIdx = 0; BodyIdx < Bodies.Num(); ++BodyIdx)
		{
			if (FBodyInstance* BodyInst = Bodies[BodyIdx])
			{
				if (UBodySetup * PhysAssetBodySetup = PhysAsset->SkeletalBodySetups[BodyIdx])
				{
					if (PhysAssetBodySetup->PhysicsType == EPhysicsType::PhysType_Default)
					{
						BodyInst->SetInstanceSimulatePhysics(bSimulate, false, true);
					}
				}
			}
		}
	}

	if(IsSimulatingPhysics())
	{
		SetRootBodyIndex(RootBodyData.BodyIndex);	//Update the root body data cache in case animation has moved root body relative to root joint
	}

 	UpdateEndPhysicsTickRegisteredState();
 	UpdateClothTickRegisteredState();
}

void USkeletalMeshComponent::OnComponentCollisionSettingsChanged(bool bUpdateOverlaps)
{
	for(int32 i=0; i<Bodies.Num(); i++)
	{
		if (ensure(Bodies[i]))
		{
			Bodies[i]->UpdatePhysicsFilterData();
		}
	}

	if (SceneProxy && !SceneProxy->IsNaniteMesh())
	{
		// TODO: Nanite-Skinning
		((FSkeletalMeshSceneProxy*)SceneProxy)->SetCollisionEnabled_GameThread(IsCollisionEnabled());
	}

	Super::OnComponentCollisionSettingsChanged(bUpdateOverlaps);
}

void USkeletalMeshComponent::AddRadialImpulse(FVector Origin, float Radius, float Strength, ERadialImpulseFalloff Falloff, bool bVelChange)
{
	if(bIgnoreRadialImpulse)
	{
		return;
	}

	PendingRadialForces.Emplace(Origin, Radius, Strength, Falloff, bVelChange, FPendingRadialForces::EType::AddImpulse);
	
	const float StrengthPerMass = Strength / FMath::Max(GetMass(), UE_KINDA_SMALL_NUMBER);
	for(FBodyInstance* Body : Bodies)
	{
		const float StrengthPerBody = bVelChange ? Strength : (StrengthPerMass * Body->GetBodyMass());
		Body->AddRadialImpulseToBody(Origin, Radius, StrengthPerBody, Falloff, bVelChange);
	}
}



void USkeletalMeshComponent::AddRadialForce(FVector Origin, float Radius, float Strength, ERadialImpulseFalloff Falloff, bool bAccelChange)
{
	if(bIgnoreRadialForce)
	{
		return;
	}

	PendingRadialForces.Emplace(Origin, Radius, Strength, Falloff, bAccelChange, FPendingRadialForces::EType::AddForce);

	const float StrengthPerMass = Strength / FMath::Max(GetMass(), UE_KINDA_SMALL_NUMBER);
	for (FBodyInstance* Body : Bodies)
	{
		const float StrengthPerBody = bAccelChange ? Strength : (StrengthPerMass * Body->GetBodyMass());
		Body->AddRadialForceToBody(Origin, Radius, StrengthPerBody, Falloff, bAccelChange);
	}
}

void USkeletalMeshComponent::WakeAllRigidBodies()
{
	for (int32 i=0; i < Bodies.Num(); i++)
	{
		FBodyInstance* BI = Bodies[i];
		check(BI);
		BI->WakeInstance();
	}
}

void USkeletalMeshComponent::PutAllRigidBodiesToSleep()
{
	for (int32 i=0; i < Bodies.Num(); i++)
	{
		FBodyInstance* BI = Bodies[i];
		check(BI);
		BI->PutInstanceToSleep();
	}
}


bool USkeletalMeshComponent::IsAnyRigidBodyAwake()
{
	bool bAwake = false;

	// ..iterate over each body to find any that are awak
	for(int32 i=0; i<Bodies.Num(); i++)
	{
		FBodyInstance* BI = Bodies[i];
		check(BI);
		if(BI->IsInstanceAwake())
		{
			// Found an awake one - so mesh is considered 'awake'
			bAwake = true;
			break;
		}
	}

	return bAwake;
}


void USkeletalMeshComponent::SetAllPhysicsLinearVelocity(FVector NewVel, bool bAddToCurrent)
{
	for (int32 i=0; i < Bodies.Num(); i++)
	{
		FBodyInstance* BodyInst = Bodies[i];
		check(BodyInst);
		BodyInst->SetLinearVelocity(NewVel, bAddToCurrent);
	}
}

void USkeletalMeshComponent::SetAllPhysicsAngularVelocityInRadians(FVector const& NewAngVel, bool bAddToCurrent)
{
	if(RootBodyData.BodyIndex != INDEX_NONE && RootBodyData.BodyIndex < Bodies.Num())
	{
		// Find the root actor. We use its location as the center of the rotation.
		FBodyInstance* RootBodyInst = Bodies[ RootBodyData.BodyIndex ];
		check(RootBodyInst);
		FTransform RootTM = RootBodyInst->GetUnrealWorldTransform();

		FVector RootPos = RootTM.GetLocation();

		// Iterate over each bone, updating its velocity
		for (int32 i = 0; i < Bodies.Num(); i++)
		{
			FBodyInstance* const BI = Bodies[i];
			check(BI);

			BI->SetAngularVelocityInRadians(NewAngVel, bAddToCurrent);
		}
	}
}

void USkeletalMeshComponent::SetAllPhysicsPosition(FVector NewPos)
{
	if(RootBodyData.BodyIndex != INDEX_NONE && RootBodyData.BodyIndex < Bodies.Num())
	{
		// calculate the deltas to get the root body to NewPos
		FBodyInstance* RootBI = Bodies[RootBodyData.BodyIndex];
		check(RootBI);
		if(RootBI->IsValidBodyInstance())
		{
			// move the root body
			FTransform RootBodyTM = RootBI->GetUnrealWorldTransform();
			FVector DeltaLoc = NewPos - RootBodyTM.GetLocation();
			RootBodyTM.SetTranslation(NewPos);
			RootBI->SetBodyTransform(RootBodyTM, ETeleportType::TeleportPhysics);

#if DO_CHECK
			FVector RelativeVector = (RootBI->GetUnrealWorldTransform().GetLocation() - NewPos);
			check(RelativeVector.SizeSquared() < 1.f);
#endif

			// apply the delta to all the other bodies
			for (int32 i = 0; i < Bodies.Num(); i++)
			{
				if (i != RootBodyData.BodyIndex)
				{
					FBodyInstance* BI = Bodies[i];
					check(BI);

					FTransform BodyTM = BI->GetUnrealWorldTransform();
					BodyTM.SetTranslation(BodyTM.GetTranslation() + DeltaLoc);
					BI->SetBodyTransform(BodyTM, ETeleportType::TeleportPhysics);
				}
			}

			// Move component to new physics location
			SyncComponentToRBPhysics();
		}
	}
}

void USkeletalMeshComponent::SetAllPhysicsRotation(FRotator NewRot)
{
	SetAllPhysicsRotation(NewRot.Quaternion());
}

void USkeletalMeshComponent::SetAllPhysicsRotation(const FQuat& NewRot)
{
#if ENABLE_NAN_DIAGNOSTIC
	if (NewRot.ContainsNaN())
	{
		logOrEnsureNanError(TEXT("USkeletalMeshComponent::SetAllPhysicsRotation found NaN in parameter NewRot: %s"), *NewRot.ToString());
	}
#endif
	if(RootBodyData.BodyIndex != INDEX_NONE && RootBodyData.BodyIndex < Bodies.Num())
	{
		// calculate the deltas to get the root body to NewRot
		FBodyInstance* RootBI = Bodies[RootBodyData.BodyIndex];
		check(RootBI);
		if(RootBI->IsValidBodyInstance())
		{
			// move the root body
			FTransform RootBodyTM = RootBI->GetUnrealWorldTransform();
			FQuat DeltaQuat = RootBodyTM.GetRotation().Inverse() * NewRot;
			RootBodyTM.SetRotation(NewRot);
			RootBI->SetBodyTransform(RootBodyTM, ETeleportType::TeleportPhysics);

			// apply the delta to all the other bodies
			for (int32 i = 0; i < Bodies.Num(); i++)
			{
				if (i != RootBodyData.BodyIndex)
				{
					FBodyInstance* BI = Bodies[i];
					check(BI);

					FTransform BodyTM = BI->GetUnrealWorldTransform();
					BodyTM.SetRotation(BodyTM.GetRotation() * DeltaQuat);
					BI->SetBodyTransform( BodyTM, ETeleportType::TeleportPhysics );
				}
			}

			// Move component to new physics location
			SyncComponentToRBPhysics();
		}
	}
}

void USkeletalMeshComponent::ApplyDeltaToAllPhysicsTransforms(const FVector& DeltaLocation, const FQuat& DeltaRotation)
{
	if(RootBodyData.BodyIndex != INDEX_NONE && RootBodyData.BodyIndex < Bodies.Num())
	{
		// calculate the deltas to get the root body to NewRot
		FBodyInstance* RootBI = Bodies[RootBodyData.BodyIndex];
		check(RootBI);
		if(RootBI->IsValidBodyInstance())
		{
			// move the root body
			FTransform RootBodyTM = RootBI->GetUnrealWorldTransform();
			RootBodyTM.SetRotation(DeltaRotation * RootBodyTM.GetRotation());
			RootBodyTM.SetTranslation(RootBodyTM.GetTranslation() + DeltaLocation);
			RootBI->SetBodyTransform(RootBodyTM, ETeleportType::TeleportPhysics);

			// apply the delta to all the other bodies
			for (int32 i = 0; i < Bodies.Num(); i++)
			{
				if (i != RootBodyData.BodyIndex)
				{
					FBodyInstance* BI = Bodies[i];
					check(BI);

					FTransform BodyTM = BI->GetUnrealWorldTransform();
					BodyTM.SetRotation(DeltaRotation * BodyTM.GetRotation());
					BodyTM.SetTranslation(BodyTM.GetTranslation() + DeltaLocation);
					BI->SetBodyTransform( BodyTM, ETeleportType::TeleportPhysics );
				}
			}

			// Move component to new physics location
			SyncComponentToRBPhysics();
		}
	}
}

void USkeletalMeshComponent::SetPhysMaterialOverride(UPhysicalMaterial* NewPhysMaterial)
{
	// Single-body case - just use PrimComp code.
	UPrimitiveComponent::SetPhysMaterialOverride(NewPhysMaterial);

	// Now update any child bodies
	for( int32 i = 0; i < Bodies.Num(); i++ )
	{
		FBodyInstance* BI = Bodies[i];
		BI->UpdatePhysicalMaterials();
	}
}

void USkeletalMeshComponent::SetEnableGravity(bool bGravityEnabled)
{
	if (!bEnablePhysicsOnDedicatedServer && IsRunningDedicatedServer())
	{
		return;
	}

	BodyInstance.bEnableGravity = bGravityEnabled;

	if (UPhysicsAsset * PhysAsset = GetPhysicsAsset())
	{
		for (int32 BodyIdx = 0; BodyIdx < Bodies.Num(); ++BodyIdx)
		{
			if (FBodyInstance* BodyInst = Bodies[BodyIdx])
			{
				if (UBodySetup * PhysAssetBodySetup = PhysAsset->SkeletalBodySetups[BodyIdx])
				{
					bool bUseGravityEnabled = bGravityEnabled;
					
					//If the default body instance has gravity turned off then turning it ON for skeletal mesh component does not turn the instance on
					if(bUseGravityEnabled && !PhysAssetBodySetup->DefaultInstance.bEnableGravity)	
					{
						bUseGravityEnabled = false;
					}
				
					BodyInst->SetEnableGravity(bUseGravityEnabled);
				}
			}
		}
	}
}

bool USkeletalMeshComponent::IsGravityEnabled() const
{
	return BodyInstance.bEnableGravity;
}

void USkeletalMeshComponent::OnConstraintBrokenWrapper(int32 ConstraintIndex)
{
	OnConstraintBroken.Broadcast(ConstraintIndex);
}

void USkeletalMeshComponent::OnPlasticDeformationWrapper(int32 ConstraintIndex)
{
	OnPlasticDeformation.Broadcast(ConstraintIndex);
}

void USkeletalMeshComponent::InitCollisionRelationships()
{
	if (UPhysicsAsset* const PhysicsAsset = GetPhysicsAsset())
	{
		int32 NumDisabledCollisions = PhysicsAsset->CollisionDisableTable.Num();
		if (NumDisabledCollisions)
		{
			TMap<FPhysicsActorHandle, TArray< FPhysicsActorHandle > > DisabledCollisions;
			for (auto& Elem : PhysicsAsset->CollisionDisableTable)
			{		
				// @question : PhysicsAsset->CollisionDisableTable should contain direct indices into the Bodies list?
				//             I saw some OOB errors in a baked build that seemed to indicate that is not the case.

				int32 SourceIndex = Elem.Key.Indices[0];
				int32 TargetIndex = Elem.Key.Indices[1];
				bool bDoCollide = !Elem.Value;
				if (0 <= SourceIndex && SourceIndex < Bodies.Num())
				{
					if (auto* SourceBody = Bodies[SourceIndex])
					{
						if (0 <= TargetIndex && TargetIndex < Bodies.Num())
						{
							if (auto* TargetBody = Bodies[TargetIndex])
							{
								if (FPhysicsActorHandle SourceHandle = SourceBody->GetPhysicsActor())
								{
									if (FPhysicsActorHandle TargetHandle = TargetBody->GetPhysicsActor())
									{
										if (!DisabledCollisions.Contains(SourceHandle))
										{
											DisabledCollisions.Add(SourceHandle, TArray<FPhysicsActorHandle>());
											DisabledCollisions[SourceHandle].Reserve(NumDisabledCollisions);
										}

										checkSlow(!DisabledCollisions[SourceHandle].Contains(TargetHandle));
										DisabledCollisions[SourceHandle].Add(TargetHandle);
									}
								}
							}
						}
					}
				}
			}
			FPhysicsCommand::ExecuteWrite(this, [&]()
				{
					FChaosEngineInterface::AddDisabledCollisionsFor_AssumesLocked(DisabledCollisions);
				});
		}
	}
}

void USkeletalMeshComponent::TermCollisionRelationships()
{
	if (UPhysicsAsset* const PhysicsAsset = GetPhysicsAsset())
	{
		int32 NumDisabledCollisions = PhysicsAsset->CollisionDisableTable.Num();
		if (NumDisabledCollisions)
		{
			TArray< FPhysicsActorHandle > CollisionRelationships;
			CollisionRelationships.Reserve(Bodies.Num());

			for (auto& Body : Bodies)
			{
				if (Body)
				{
					if (FPhysicsActorHandle Handle = Body->GetPhysicsActor())
					{
						CollisionRelationships.Add(Handle);
					}
				}
			}

			FPhysicsCommand::ExecuteWrite(this, [&]()
			{
				FChaosEngineInterface::RemoveDisabledCollisionsFor_AssumesLocked(CollisionRelationships);
			});
		}
	}
}


DECLARE_CYCLE_STAT(TEXT("Init Articulated"), STAT_InitArticulated, STATGROUP_Physics);

int32 USkeletalMeshComponent::FindRootBodyIndex() const
{
	// Find root physics body
	int32 RootBodyIndex = RootBodyData.BodyIndex;
	if(RootBodyIndex == INDEX_NONE && GetSkeletalMeshAsset())
	{
		if(const UPhysicsAsset* PhysicsAsset = GetPhysicsAsset())
		{
			for (int32 i = 0; i< GetSkeletalMeshAsset()->GetRefSkeleton().GetNum(); i++)
			{
				int32 BodyInstIndex = PhysicsAsset->FindBodyIndex(GetSkeletalMeshAsset()->GetRefSkeleton().GetBoneName(i));
				if (BodyInstIndex != INDEX_NONE)
				{
					RootBodyIndex = BodyInstIndex;
					break;
				}
			}
		}
	}

	return RootBodyIndex;
}

static int32 bAllowNotForDedServerPhysicsAssets = 1;
static FAutoConsoleVariableRef CVarAllowNotForDedServerPhysicsAssets(
	TEXT("p.AllowNotForDedServerPhysicsAssets"),
	bAllowNotForDedServerPhysicsAssets,
	TEXT("Allow 'Not For Dedicated Server' flag on PhysicsAssets\n")
	TEXT("0: ignore flag, 1: obey flag (default)"),
	ECVF_Default);

void USkeletalMeshComponent::InitArticulated(FPhysScene* PhysScene)
{
	SCOPE_CYCLE_COUNTER(STAT_InitArticulated);

	UPhysicsAsset* const PhysicsAsset = GetPhysicsAsset();

	if (PhysScene == nullptr || PhysicsAsset == nullptr || GetSkeletalMeshAsset() == nullptr || !ShouldCreatePhysicsState())
	{
		return;
	}

	if (Bodies.Num() > 0)
	{
		UE_LOGF(LogSkeletalMesh, Log, "USkeletalMeshComponent::InitArticulated : Bodies already created (%ls) - call TermArticulated first.", *GetPathName());
		return;
	}

	// Skip if not desired on dedicated server
	UWorld* World = GetWorld();
	if (PhysicsAsset->bNotForDedicatedServer && World && (World->GetNetMode() == NM_DedicatedServer) && bAllowNotForDedServerPhysicsAssets)
	{
		UE_LOGF(LogSkeletalMesh, Verbose, "Skipping PhysicsAsset creation on dedicated server (%ls : %ls) %ls", *GetNameSafe(GetOuter()), *GetName(), *PhysicsAsset->GetName());
		return;
	}

	FVector Scale3D = GetComponentTransform().GetScale3D();

	// Find root physics body
	RootBodyData.BodyIndex = INDEX_NONE;	//Reset the root body index just in case we need to refind a new one
	const int32 RootBodyIndex = FindRootBodyIndex();

	if (RootBodyIndex == INDEX_NONE)
	{
		UE_LOGF(LogSkeletalMesh, Log, "USkeletalMeshComponent::InitArticulated : Could not find root physics body: '%ls'", *GetPathName());
		return;
	}

	// Set up the map from skelmeshcomp ID to collision disable table
	int32 NumShapes = 0;
	const int32 NumBodies = PhysicsAsset->SkeletalBodySetups.Num();
	for (int32 BodyIndex = 0; BodyIndex < NumBodies; ++BodyIndex)
	{
		if (PhysicsAsset->SkeletalBodySetups[BodyIndex])
		{
			NumShapes += PhysicsAsset->SkeletalBodySetups[BodyIndex]->AggGeom.GetElementCount();
		}
	}

	if (!Aggregate.IsValid() && NumShapes > RagdollAggregateThreshold&& NumShapes <= AggregateMaxSize)
	{
		Aggregate = FPhysicsInterface::CreateAggregate(PhysicsAsset->SkeletalBodySetups.Num());
	}
	else if (Aggregate.IsValid() && NumShapes > AggregateMaxSize)
	{
		UE_LOGF(LogSkeletalMesh, Log, "USkeletalMeshComponent::InitArticulated : Too many shapes to create aggregate, Max: %u, This: %d", AggregateMaxSize, NumShapes);
	}

	InstantiatePhysicsAsset(*PhysicsAsset, Scale3D, Bodies, Constraints, PhysScene, this, RootBodyIndex, Aggregate);
	for (int32 BodyIndex = 0; BodyIndex < NumBodies; ++BodyIndex)
	{
		if (FBodyInstance* Body = Bodies[BodyIndex])
		{
			if (!Body->GetPhysicsActor())
			{
				continue;
			}

			Chaos::FPhysicsObject* PhysicsObject = Body->GetPhysicsActor()->GetPhysicsObject();
			if (UBodySetup* Setup = Body->GetBodySetup())
			{
				Chaos::FPhysicsObjectInterface::SetName(PhysicsObject, Setup->BoneName);
			}
			Chaos::FPhysicsObjectInterface::SetId(PhysicsObject, BodyIndex);
		}
	}

	// now update root body index because body has BodySetup now
	SetRootBodyIndex(RootBodyIndex);

	InitCollisionRelationships();

	// Update Flag
	PrevRootBoneMatrix = GetBoneMatrix(0); // save the root bone transform

	PrevRootRigidBodyScale = FVector3f(GetRelativeScale3D()); //Update the root rigid body scale. 

	// pre-compute cloth teleport thresholds for performance
	ClothTeleportDistThresholdSquared = UE::ClothingSimulation::TeleportHelpers::ComputeTeleportDistanceThresholdSquared(TeleportDistanceThreshold);
	ClothTeleportCosineThresholdInRad = UE::ClothingSimulation::TeleportHelpers::ComputeTeleportCosineRotationThreshold(TeleportRotationThreshold);
}

TAutoConsoleVariable<int32> CVarEnableRagdollPhysics(TEXT("p.RagdollPhysics"), 1, TEXT("If 1, ragdoll physics will be used. Otherwise just root body is simulated"));

void USkeletalMeshComponent::InstantiatePhysicsAsset(const UPhysicsAsset& PhysAsset, const FVector& Scale3D, TArray<FBodyInstance*>& OutBodies, TArray<FConstraintInstance*>& OutConstraints, FPhysScene* PhysScene, USkeletalMeshComponent* OwningComponent, int32 UseRootBodyIndex, const FPhysicsAggregateHandle& UseAggregate) const
{
	auto BoneTMCallable = [this](int32 BoneIndex)
	{
		const bool bExpectedBoneTransformsValidity = IsPreRegistering() || IsPreRegistered() || IsRegistered();
		checkf(bExpectedBoneTransformsValidity == AreBoneTransformsValid(), TEXT("Component bone transforms are inconsistent with the component's registration state."));
		return GetBoneTransform(BoneIndex);
	};

	InstantiatePhysicsAsset_Internal(PhysAsset, Scale3D, OutBodies, OutConstraints, BoneTMCallable, PhysScene, OwningComponent, UseRootBodyIndex, UseAggregate);
}

void USkeletalMeshComponent::InstantiatePhysicsAssetBodies(const UPhysicsAsset& PhysAsset, TArray<FBodyInstance*>& OutBodies, FPhysScene* PhysScene /*= nullptr*/, USkeletalMeshComponent* OwningComponent /*= nullptr*/, int32 UseRootBodyIndex /*= INDEX_NONE*/, const FPhysicsAggregateHandle& UseAggregate /*= FPhysicsAggregateHandle()*/) const
{
	auto BoneTMCallable = [this](int32 BoneIndex)
	{
		return GetBoneTransform(BoneIndex);
	};

	InstantiatePhysicsAssetBodies_Internal(PhysAsset, OutBodies, BoneTMCallable, nullptr, PhysScene, OwningComponent, UseRootBodyIndex, UseAggregate);
}

void USkeletalMeshComponent::InstantiatePhysicsAssetBodiesRefPose(const UPhysicsAsset& PhysAsset, TArray<FBodyInstance*>& OutBodies, FPhysScene* PhysScene /*= nullptr*/, USkeletalMeshComponent* OwningComponent /*= nullptr*/, int32 UseRootBodyIndex /*= INDEX_NONE*/, const FPhysicsAggregateHandle& UseAggregate /*= FPhysicsAggregateHandle()*/) const
{
	if (GetSkeletalMeshAsset())
	{
		const FReferenceSkeleton& RefSkeleton = GetSkeletalMeshAsset()->GetRefSkeleton();

		// Create the bodies posed in component-space with the ref pose
		TArray<FTransform> CachedTransforms;
		TArray<bool> CachedTransformReady;
		CachedTransforms.SetNumUninitialized(RefSkeleton.GetRefBonePose().Num());
		CachedTransformReady.SetNumZeroed(RefSkeleton.GetRefBonePose().Num());

		auto BoneTMCallable = [&](int32 BoneIndex)
			{
				return FAnimationRuntime::GetComponentSpaceTransformWithCache(RefSkeleton, RefSkeleton.GetRefBonePose(), BoneIndex, CachedTransforms, CachedTransformReady);
			};

		InstantiatePhysicsAssetBodies_Internal(PhysAsset, OutBodies, BoneTMCallable, nullptr, PhysScene, OwningComponent, UseRootBodyIndex, UseAggregate);
	}
}

void USkeletalMeshComponent::InstantiatePhysicsAssetRefPose(const UPhysicsAsset& PhysAsset, const FVector& Scale3D, TArray<FBodyInstance*>& OutBodies, TArray<FConstraintInstance*>& OutConstraints, FPhysScene* PhysScene /*= nullptr*/, USkeletalMeshComponent* OwningComponent /*= nullptr*/, int32 UseRootBodyIndex /*= INDEX_NONE*/, const FPhysicsAggregateHandle& UseAggregate, bool bCreateBodiesInRefPose) const
{
	if(GetSkeletalMeshAsset())
	{
		const FReferenceSkeleton& RefSkeleton = GetSkeletalMeshAsset()->GetRefSkeleton();

		if (bCreateBodiesInRefPose)
		{
			// Create the bodies posed in component-space with the ref pose
			TArray<FTransform> CachedTransforms;
			TArray<bool> CachedTransformReady;
			CachedTransforms.SetNumUninitialized(RefSkeleton.GetRefBonePose().Num());
			CachedTransformReady.SetNumZeroed(RefSkeleton.GetRefBonePose().Num());

			auto BoneTMCallable = [&](int32 BoneIndex)
			{
				return FAnimationRuntime::GetComponentSpaceTransformWithCache(RefSkeleton, RefSkeleton.GetRefBonePose(), BoneIndex, CachedTransforms, CachedTransformReady);
			};

			InstantiatePhysicsAsset_Internal(PhysAsset, Scale3D, OutBodies, OutConstraints, BoneTMCallable, PhysScene, OwningComponent, UseRootBodyIndex, UseAggregate);
		}
		else
		{
			// Create the bodies with each body in its local ref pose (all bodies will be on top of each other close to the origin, but should still have the correct scale)
			auto BoneTMCallable = [&](int32 BoneIndex)
			{
				if (RefSkeleton.IsValidIndex(BoneIndex))
				{
					return RefSkeleton.GetRefBonePose()[BoneIndex];
				}
				return FTransform::Identity;
			};;

			InstantiatePhysicsAsset_Internal(PhysAsset, Scale3D, OutBodies, OutConstraints, BoneTMCallable, PhysScene, OwningComponent, UseRootBodyIndex, UseAggregate);
		}
	}
}

void USkeletalMeshComponent::InstantiatePhysicsAssetBodies_Internal(const UPhysicsAsset& PhysAsset, TArray<FBodyInstance*>& OutBodies, TFunctionRef<FTransform(int32)> BoneTransformGetter, TMap<FName, FBodyInstance*>* OutNameToBodyMap, FPhysScene* PhysScene /*= nullptr*/, USkeletalMeshComponent* OwningComponent /*= nullptr*/, int32 UseRootBodyIndex /*= INDEX_NONE*/, const FPhysicsAggregateHandle& UseAggregate) const
{
	const int32 NumOutBodies = PhysAsset.SkeletalBodySetups.Num();

	// Create all the OutBodies.
	check(OutBodies.Num() == 0);
	OutBodies.AddZeroed(NumOutBodies);
	
	if (OutNameToBodyMap)
	{
		OutNameToBodyMap->Reserve(NumOutBodies);
	}

	for(int32 BodyIdx = 0; BodyIdx < NumOutBodies; BodyIdx++)
	{
		UBodySetup* PhysicsAssetBodySetup = PhysAsset.SkeletalBodySetups[BodyIdx];
		if (!ensure(PhysicsAssetBodySetup))
		{
			continue;
		}
		OutBodies[BodyIdx] = new FBodyInstance;
		FBodyInstance* BodyInst = OutBodies[BodyIdx];
		check(BodyInst);

		// Get transform of bone by name.
		int32 BoneIndex = GetBoneIndex(PhysicsAssetBodySetup->BoneName);
		if(BoneIndex != INDEX_NONE)
		{
			// Copy body setup default instance properties
			BodyInst->CopyBodyInstancePropertiesFrom(&PhysicsAssetBodySetup->DefaultInstance);
			// we don't allow them to use this in editor. For physics asset, this set up is overridden by Physics Type. 
			// but before we hide in the detail customization, we saved with this being true, causing the simulate always happens for some OutBodies
			// so adding initialization here to disable this. 
			// to check, please check BodySetupDetails.cpp, if (ChildProperty->GetProperty()->GetName() == TEXT("bSimulatePhysics"))
			// we hide this property, so it should always be false initially. 
			// this is not true for all other BodyInstance, but for physics assets it is true. 
			BodyInst->bSimulatePhysics = false;
			BodyInst->InstanceBodyIndex = BodyIdx; // Set body index 
			BodyInst->InstanceBoneIndex = BoneIndex; // Set bone index

			BodyInst->bStartAwake = UseRootBodyIndex >= 0 ? BodyInstance.bStartAwake : true;	//We don't allow customization here. Just use whatever the component is set to

			//Copying code from BodyInstance here, every body instance get same iteration count
			BodyInst->PositionSolverIterationCount = BodyInstance.PositionSolverIterationCount;
			BodyInst->VelocitySolverIterationCount = BodyInstance.VelocitySolverIterationCount;
			BodyInst->ProjectionSolverIterationCount = BodyInstance.ProjectionSolverIterationCount;

			if(BodyIdx == UseRootBodyIndex)
			{
				BodyInst->DOFMode = BodyInstance.DOFMode;
				BodyInst->CustomDOFPlaneNormal = BodyInstance.CustomDOFPlaneNormal;
				BodyInst->bLockXTranslation = BodyInstance.bLockXTranslation;
				BodyInst->bLockYTranslation = BodyInstance.bLockYTranslation;
				BodyInst->bLockZTranslation = BodyInstance.bLockZTranslation;
				BodyInst->bLockXRotation = BodyInstance.bLockXRotation;
				BodyInst->bLockYRotation = BodyInstance.bLockYRotation;
				BodyInst->bLockZRotation = BodyInstance.bLockZRotation;
				BodyInst->bLockTranslation = BodyInstance.bLockTranslation;
				BodyInst->bLockRotation = BodyInstance.bLockRotation;

				BodyInst->COMNudge += BodyInstance.COMNudge;
			}
			else
			{
				BodyInst->DOFMode = EDOFMode::None;
				if(PhysScene != nullptr && !CVarEnableRagdollPhysics.GetValueOnAnyThread())	//We only limit creation of the global physx scenes and not assets related to immedate mode
				{
					continue;
				}
			}

			// Create physics body instance.
			FTransform BoneTransform = BoneTransformGetter(BoneIndex);

			FInitBodySpawnParams SpawnParams(OwningComponent);

			if(OwningComponent == nullptr)
			{
				//special case where we don't use the skel mesh, but we still want to do certain logic like skeletal mesh
				SpawnParams.bStaticPhysics = false;
				SpawnParams.bPhysicsTypeDeterminesSimulation = true;
			}

			SpawnParams.Aggregate = UseAggregate;
			BodyInst->InitBody(PhysicsAssetBodySetup, BoneTransform, OwningComponent, PhysScene, SpawnParams);

			if (OutNameToBodyMap)
			{
				OutNameToBodyMap->Add(PhysicsAssetBodySetup->BoneName, BodyInst);
			}
		}
	}
}

void USkeletalMeshComponent::InstantiatePhysicsAsset_Internal(const UPhysicsAsset& PhysAsset, const FVector& Scale3D, TArray<FBodyInstance*>& OutBodies, TArray<FConstraintInstance*>& OutConstraints, TFunctionRef<FTransform(int32)> BoneTransformGetter, FPhysScene* PhysScene /*= nullptr*/, USkeletalMeshComponent* OwningComponent /*= nullptr*/, int32 UseRootBodyIndex /*= INDEX_NONE*/, const FPhysicsAggregateHandle& UseAggregate) const
{
	const float ActualScale = Scale3D.GetAbsMin();
	const float Scale = ActualScale == 0.f ? UE_KINDA_SMALL_NUMBER : ActualScale;

	TMap<FName, FBodyInstance*> NameToBodyMap;

	// Create all the OutBodies.
	InstantiatePhysicsAssetBodies_Internal(PhysAsset, OutBodies, BoneTransformGetter, &NameToBodyMap, PhysScene, OwningComponent, UseRootBodyIndex, UseAggregate);

	if(PhysScene && Aggregate.IsValid())
	{
		PhysScene->AddAggregateToScene(Aggregate);
	}
	
	// Create all the OutConstraints.
	check(OutConstraints.Num() == 0);
	int32 NumOutConstraints = PhysAsset.ConstraintSetup.Num();
	OutConstraints.AddZeroed(NumOutConstraints);
	for(int32 ConstraintIdx = 0; ConstraintIdx < NumOutConstraints; ConstraintIdx++)
	{
		const UPhysicsConstraintTemplate* OutConstraintsetup = PhysAsset.ConstraintSetup[ConstraintIdx];
		OutConstraints[ConstraintIdx] = new FConstraintInstance;
		FConstraintInstance* ConInst = OutConstraints[ConstraintIdx];
		check(ConInst);
		ConInst->CopyConstraintParamsFrom(&OutConstraintsetup->DefaultInstance);
		ConInst->ConstraintIndex = ConstraintIdx; // Set the ConstraintIndex property in the ConstraintInstance.
		ConInst->PhysScene = PhysScene;
#if WITH_EDITOR
		UWorld* World = GetWorld();
		if(World && World->IsGameWorld())
		{
			//In the editor we may be currently editing the physics asset, so make sure to use the default profile
			OutConstraintsetup->ApplyConstraintProfile(NAME_None, *ConInst, /*bDefaultIfNotFound=*/true);
		}
#endif

		// Get OutBodies we want to joint
		FBodyInstance* Body1 = NameToBodyMap.FindRef(ConInst->ConstraintBone1);
		FBodyInstance* Body2 = NameToBodyMap.FindRef(ConInst->ConstraintBone2);

		// If we have 2, joint 'em
		if(Body1 && Body2)
		{
			// Validates the body. OutBodies could be invalid due to outdated PhysAssets / bad constraint bone (or body) names.
			auto ValidateBody = [this](const FBodyInstance* InBody, const FName& InBoneName)
			{
				if(!InBody->IsValidBodyInstance())
				{
					// Disable log for now.
					// UE_LOGF(LogSkeletalMesh, Warning, "USkeletalMeshComponent::InitArticulated : Unable to initialize constraint (%ls) -  Body Invalid %ls.", *(this->GetPathName()), *(InBoneName.ToString()));
					return false;
				}

				return true;
			};

			// Applies the adjusted / relative scale of the body instance.
			// Also, remove component scale as it will be reapplied in InitConstraint.
			// GetBoneTransform already accounts for component scale.
			auto ScalePosition = [](const FBodyInstance* InBody, const float InScale, FVector& OutPosition)
			{
				const FBodyInstance& DefaultBody = InBody->GetBodySetup()->DefaultInstance;
				const FVector ScaledDefaultBodyScale = DefaultBody.Scale3D * InScale;
				const FVector AdjustedBodyScale = InBody->Scale3D * ScaledDefaultBodyScale.Reciprocal();
				OutPosition *= AdjustedBodyScale;
			};

			// Do this separately so both are logged if invalid.
			const bool Body1Valid = ValidateBody(Body1, ConInst->ConstraintBone1);
			const bool Body2Valid = ValidateBody(Body2, ConInst->ConstraintBone2);

			if(Body1Valid && Body2Valid)
			{
				ScalePosition(Body1, Scale, ConInst->Pos1);
				ScalePosition(Body2, Scale, ConInst->Pos2);
				ConInst->InitConstraint(Body1, Body2, Scale, OwningComponent
					, OwningComponent ? FOnConstraintBroken::CreateUObject(OwningComponent, &USkeletalMeshComponent::OnConstraintBrokenWrapper) : FOnConstraintBroken()
					, OwningComponent ? FOnPlasticDeformation::CreateUObject(OwningComponent, &USkeletalMeshComponent::OnPlasticDeformationWrapper) : FOnPlasticDeformation());
			}
		}
	}
}

void USkeletalMeshComponent::TermArticulated()
{
	const bool bIsGameThread = IsInGameThread();
	const bool bIsAsyncDestroyPhysicsStateRunning = IsAsyncDestroyPhysicsStateRunning();
	const bool bPrepareAsyncPayload = bIsAsyncDestroyPhysicsStateRunning && bIsGameThread;
	const bool bUseAsyncPayload = bIsAsyncDestroyPhysicsStateRunning && !bIsGameThread;
	check(!bPrepareAsyncPayload || !bUseAsyncPayload);

	if (bIsGameThread)
	{
		ResetRootBodyIndex();

		TermCollisionRelationships();
	}
	
	auto TermArticulatedInternal = [this, bPrepareAsyncPayload, bUseAsyncPayload]()
	{
		TArray<FBodyInstance*>& InOutBodies = bUseAsyncPayload ? AsyncDestroyPhysicsStatePayload : Bodies;
		TArray<FConstraintInstance*>& InOutConstraints = bUseAsyncPayload ? AsyncDestroyConstraintsPayload : Constraints;
		FPhysicsAggregateHandle& InOutAggregate = bUseAsyncPayload ? AggregatePayload : Aggregate;

		// We shut down the physics for each body and constraint here. The actual UObjects will get GC'd.
		for (int32 i = 0; i < InOutBodies.Num(); i++)
		{
			if (ensure(InOutBodies[i]))
			{
				// There seems to be scenarios where there is a body that still has an active weld parent.
				if (!ensure(InOutBodies[i]->WeldParent == nullptr))
				{
					InOutBodies[i]->WeldParent->UnWeld(InOutBodies[i]);
				}

				// Asynchronous destruction is done in OnAsyncDestroyPhysicsState
				if (!bPrepareAsyncPayload)
				{
					InOutBodies[i]->TermBody();
					delete InOutBodies[i];
				}
			}
		}

		if (!bPrepareAsyncPayload)
		{
			for (int32 i = 0; i < InOutConstraints.Num(); i++)
			{
				check(InOutConstraints[i]);
				InOutConstraints[i]->TermConstraint();
				delete InOutConstraints[i];
			}

			InOutConstraints.Empty();
			InOutBodies.Empty();

			// releasing Aggregate, it shouldn't contain any Bodies now, because they are released above
			if (InOutAggregate.IsValid())
			{
				check(FPhysicsInterface::GetNumActorsInAggregate(InOutAggregate) == 0);
				FPhysicsInterface::ReleaseAggregate(InOutAggregate);
			}
		}
		else
		{
			// Prepare payload for asynchronous destruction
			check(AsyncDestroyConstraintsPayload.IsEmpty());
			AsyncDestroyConstraintsPayload = MoveTemp(Constraints);

			check(AsyncDestroyPhysicsStatePayload.IsEmpty());
			AsyncDestroyPhysicsStatePayload = MoveTemp(Bodies);

			check(!AggregatePayload.IsValid());
			AggregatePayload = MoveTemp(Aggregate);
		}
	};

	if (bPrepareAsyncPayload)
	{
		// No need to lock scene when preparing asynchronous destruction payloads
		TermArticulatedInternal();
	}
	else
	{
		FPhysicsCommand::ExecuteWrite(this, [&TermArticulatedInternal]()
		{
			TermArticulatedInternal();
		});
	}
}

bool USkeletalMeshComponent::AllowsAsyncPhysicsStateCreation() const
{
	check(FPhysScene::SupportsAsyncPhysicsStateCreation());

	// Note: AutoWeld is disabled by default for PhysicsAsset's BodyInstances
	return GEnableSkeletalMeshComponentAsyncInitBody && (!BodyInstance.bAutoWeld || !Cast<UPrimitiveComponent>(GetAttachParent()));
}

bool USkeletalMeshComponent::AllowsAsyncPhysicsStateDestruction() const
{
	check(FPhysScene::SupportsAsyncPhysicsStateDestruction());
	return AllowsAsyncPhysicsStateCreation();
}

void USkeletalMeshComponent::OnAsyncCreatePhysicsStateBegin_GameThread(TSet<UObject*>& OutRootedObjects)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(USkeletalMeshComponent::OnAsyncCreatePhysicsStateBegin_GameThread);

	if (bEnablePerPolyCollision == false)
	{
		USceneComponent::OnAsyncCreatePhysicsStateBegin_GameThread(OutRootedObjects);
	}
	else
	{
		CreateBodySetup();
		BodySetup->CreatePhysicsMeshes();
		Super::OnAsyncCreatePhysicsStateBegin_GameThread(OutRootedObjects);
	}
}

bool USkeletalMeshComponent::OnAsyncCreatePhysicsState(const UE::FTimeout& TimeOut)
{
	// Don't call Super::OnAsyncCreatePhysicsState as we don't want UActorComponent::OnAsyncCreatePhysicsState
	// to call OnCreatePhysicsState (which would call USkeletalMeshComponent::OnCreatePhysicsState).
	TRACE_CPUPROFILER_EVENT_SCOPE(USkeletalMeshComponent::OnAsyncCreatePhysicsState);
	check(AllowsAsyncPhysicsStateCreation());

	check(AreBoneTransformsValid());

	if (bEnablePerPolyCollision == false)
	{
		InitArticulated(GetWorld()->GetPhysicsScene());

		// Don't call Super (skip PrimitiveComponent)
		USceneComponent::OnCreatePhysicsState();
	}
	else
	{
		// If we're doing per poly we'll use the body instance of the primitive component
		Super::OnCreatePhysicsState();
	}

	return true;
}


void USkeletalMeshComponent::OnAsyncCreatePhysicsStateEnd_GameThread()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(USkeletalMeshComponent::OnAsyncCreatePhysicsStateEnd_GameThread);

	if (bEnablePerPolyCollision == false)
	{
		// Don't call Super (skip PrimitiveComponent)
		USceneComponent::OnAsyncCreatePhysicsStateEnd_GameThread();
#if UE_ENABLE_DEBUG_DRAWING
		SendRenderDebugPhysics();
#endif
		// Update per-frame physics quantization registration
		UE::PhysicsReplicationQuantization::NotifyComponentPhysicsState(this, /*bRegister*/true);
	}
	else
	{
		Super::OnAsyncCreatePhysicsStateEnd_GameThread();
	}

	// Notify physics created
	OnSkelMeshPhysicsCreated.Broadcast();
}

void USkeletalMeshComponent::OnAsyncDestroyPhysicsStateBegin_GameThread(TSet<UObject*>& OutRootedObjects)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(USkeletalMeshComponent::OnAsyncDestroyPhysicsStateBegin_GameThread);

	check(IsAsyncDestroyPhysicsStateRunning());

	// Update per-frame physics quantization registration
	UE::PhysicsReplicationQuantization::NotifyComponentPhysicsState(this, /*bRegister*/false);

	DestroyPhysicsStateInternal();

	Super::OnAsyncDestroyPhysicsStateBegin_GameThread(OutRootedObjects);
}

bool USkeletalMeshComponent::OnAsyncDestroyPhysicsState(const UE::FTimeout& Timeout)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(USkeletalMeshComponent::OnAsyncDestroyPhysicsState);

	if (bEnablePerPolyCollision == false)
	{
		check(Bodies.IsEmpty());
		check(Constraints.IsEmpty());
		check(!Aggregate.IsValid());

		TermArticulated();

		check(AsyncDestroyPhysicsStatePayload.IsEmpty());
		check(AsyncDestroyConstraintsPayload.IsEmpty());
		check(!AggregatePayload.IsValid());
	}

	return Super::OnAsyncDestroyPhysicsState(Timeout);
}

void USkeletalMeshComponent::TermBodiesBelow(FName ParentBoneName)
{
	UPhysicsAsset* const PhysicsAsset = GetPhysicsAsset();
	if(PhysicsAsset && GetSkeletalMeshAsset() && Bodies.Num() > 0)
	{
		check(Bodies.Num() == PhysicsAsset->SkeletalBodySetups.Num());

		// Get index of parent bone
		int32 ParentBoneIndex = GetBoneIndex(ParentBoneName);
		if(ParentBoneIndex == INDEX_NONE)
		{
			UE_LOGF(LogSkeletalMesh, Log, "TermBodiesBelow: ParentBoneName '%ls' is invalid", *ParentBoneName.ToString());
			return;
		}

		// First terminate any constraints at below this bone
		for(int32 i=0; i<Constraints.Num(); i++)
		{
			// Get bone index of constraint
			FName JointChildBoneName = Constraints[i]->GetChildBoneName();
			int32 JointBoneIndex = GetBoneIndex(JointChildBoneName);

			// If constraint has bone in mesh, and is either the parent or child of it, term it
			if(	JointBoneIndex != INDEX_NONE && (JointChildBoneName == ParentBoneName || GetSkeletalMeshAsset()->GetRefSkeleton().BoneIsChildOf(JointBoneIndex, ParentBoneIndex)) )
			{
				Constraints[i]->TermConstraint();
			}
		}

		// Then iterate over bodies looking for any which are children of supplied parent
		for(int32 i=0; i<Bodies.Num(); i++)
		{
			// Get bone index of body
			if (Bodies[i]->IsValidBodyInstance())
			{
				FName BodyName = Bodies[i]->BodySetup->BoneName;
				int32 BodyBoneIndex = GetBoneIndex(BodyName);

				// If body has bone in mesh, and is either the parent or child of it, term it
				if(	BodyBoneIndex != INDEX_NONE && (BodyName == ParentBoneName || GetSkeletalMeshAsset()->GetRefSkeleton().BoneIsChildOf(BodyBoneIndex, ParentBoneIndex)) )
				{
					Bodies[i]->TermBody();
				}
			}
		}
	}
}

float USkeletalMeshComponent::GetTotalMassBelowBone(FName InBoneName)
{
	float TotalBodyMass = 0.f;

	ForEachBodyBelow(InBoneName, /*bIncludeSelf=*/true, /*bSkipCustomPhysics=*/false, [&TotalBodyMass](FBodyInstance* BI)
	{
		TotalBodyMass += BI->GetBodyMass();
	});

	return TotalBodyMass;
}

void USkeletalMeshComponent::SetAllBodiesSimulatePhysics(bool bNewSimulate)
{
	for(int32 i=0; i<Bodies.Num(); i++)
	{
		Bodies[i]->SetInstanceSimulatePhysics(bNewSimulate, false, true);
	}

	SetRootBodyIndex(RootBodyData.BodyIndex);	//Update the root body data cache in case animation has moved root body relative to root joint

	UpdateEndPhysicsTickRegisteredState();
	UpdateClothTickRegisteredState();
}

void USkeletalMeshComponent::SetCollisionObjectType(ECollisionChannel NewChannel)
{
	SetAllBodiesCollisionObjectType(NewChannel);
}

void USkeletalMeshComponent::SetAllBodiesCollisionObjectType(ECollisionChannel NewChannel)
{
	BodyInstance.SetObjectType(NewChannel);	//children bodies use the skeletal mesh override so make sure root is set properly

	for(int32 i=0; i<Bodies.Num(); i++)
	{
		Bodies[i]->SetObjectType(NewChannel);
	}
}

void USkeletalMeshComponent::SetAllBodiesNotifyRigidBodyCollision(bool bNewNotifyRigidBodyCollision)
{
	BodyInstance.SetInstanceNotifyRBCollision(bNewNotifyRigidBodyCollision); //children bodies use the skeletal mesh override so make sure root is set properly

	for(int32 i=0; i<Bodies.Num(); i++)
	{
		Bodies[i]->SetInstanceNotifyRBCollision(bNewNotifyRigidBodyCollision);
	}
}

void USkeletalMeshComponent::SetAllBodiesBelowPhysicsDisabled(const FName& InBoneName, bool bDisabled, bool bIncludeSelf)
{
	int32 NumBodiesFound = ForEachBodyBelow(InBoneName, bIncludeSelf, /*bSkipCustomPhysicsType=*/ false, [bDisabled, this](FBodyInstance* BI)
	{
		BI->SetPhysicsDisabled(bDisabled);
		if (bDisabled == false)
		{
			FTransform BoneTransform(GetBoneMatrix(BI->InstanceBoneIndex));
			// if we re-enable it, let's make sure the body transform is up to date 
			BI->SetBodyTransform(BoneTransform, ETeleportType::TeleportPhysics, true);
		}
	});
}

void USkeletalMeshComponent::SetAllBodiesBelowLinearVelocity(const FName& InBoneName, const FVector& LinearVelocity, bool bIncludeSelf)
{
	if (const FBodyInstance* ParentBodyInstance = GetBodyInstance(InBoneName))
	{
		ForEachBodyBelow(InBoneName, bIncludeSelf, /*bSkipCustomPhysicsType=*/ false, [LinearVelocity, this](FBodyInstance* BI)
			{
				BI->SetLinearVelocity(LinearVelocity, false);
			});
	}
}

FVector USkeletalMeshComponent::GetBoneLinearVelocity(const FName& InBoneName)
{
	FVector OutVelocity = FVector::ZeroVector;

	if (const FBodyInstance* ParentBodyInstance = GetBodyInstance(InBoneName))
	{
		OutVelocity = ParentBodyInstance->GetUnrealWorldVelocity();
	}

	return OutVelocity;
}


void USkeletalMeshComponent::SetAllBodiesBelowSimulatePhysics( const FName& InBoneName, bool bNewSimulate, bool bIncludeSelf )
{
	int32 NumBodiesFound = ForEachBodyBelow(InBoneName, bIncludeSelf, /*bSkipCustomPhysicsType=*/ false, [bNewSimulate](FBodyInstance* BI)
	{
		BI->SetInstanceSimulatePhysics(bNewSimulate, false, true);
	});

	if (NumBodiesFound)
	{
		if (IsSimulatingPhysics())
		{
			SetRootBodyIndex(RootBodyData.BodyIndex);	//Update the root body data cache in case animation has moved root body relative to root joint
		}

		UpdateEndPhysicsTickRegisteredState();
		UpdateClothTickRegisteredState();
	}
}

void USkeletalMeshComponent::SetBodySimulatePhysics(const FName& InBoneName, bool bSimulate)
{
	FBodyInstance* BI = GetBodyInstance(InBoneName);
	if (BI)
	{
		BI->SetInstanceSimulatePhysics(bSimulate, false, true);

		if (IsSimulatingPhysics())
		{
			SetRootBodyIndex(RootBodyData.BodyIndex);	//Update the root body data cache in case animation has moved root body relative to root joint
		}

		UpdateEndPhysicsTickRegisteredState();
		UpdateClothTickRegisteredState();
	}
}

void USkeletalMeshComponent::SetAllMotorsAngularPositionDrive(bool bEnableSwingDrive, bool bEnableTwistDrive, bool bSkipCustomPhysicsType)
{
	UPhysicsAsset* const PhysicsAsset = GetPhysicsAsset();
	if( !PhysicsAsset )
	{
		return;
	}

	for(int32 i=0; i<Constraints.Num(); i++)
	{
		if( bSkipCustomPhysicsType )
		{
			int32 BodyIndex = PhysicsAsset->FindBodyIndex(Constraints[i]->GetChildBoneName());
			if( BodyIndex != INDEX_NONE && PhysicsAsset->SkeletalBodySetups[BodyIndex]->PhysicsType != PhysType_Default)
			{
				continue;
			}
		}

		Constraints[i]->SetOrientationDriveTwistAndSwing(bEnableTwistDrive, bEnableSwingDrive);
		Constraints[i]->SetOrientationDriveSLERP(bEnableTwistDrive || bEnableSwingDrive);
	}
}

void USkeletalMeshComponent::SetNamedMotorsAngularPositionDrive(bool bEnableSwingDrive, bool bEnableTwistDrive, const TArray<FName>& BoneNames, bool bSetOtherBodiesToComplement)
{
	UPhysicsAsset* const PhysicsAsset = GetPhysicsAsset();
	if( !PhysicsAsset )
	{
		return;
	}

	for(int32 i=0; i<Constraints.Num(); i++)
	{
		FConstraintInstance* Instance = Constraints[i];
		if( BoneNames.Contains(Instance->GetChildBoneName()) )
		{
			Constraints[i]->SetOrientationDriveTwistAndSwing(bEnableTwistDrive, bEnableSwingDrive);
			Constraints[i]->SetOrientationDriveSLERP(bEnableTwistDrive || bEnableSwingDrive);
		}
		else if( bSetOtherBodiesToComplement )
		{
			Constraints[i]->SetOrientationDriveTwistAndSwing(!bEnableTwistDrive, !bEnableSwingDrive);
			Constraints[i]->SetOrientationDriveSLERP(!(bEnableTwistDrive || bEnableSwingDrive));
		}
	}
}

void USkeletalMeshComponent::SetNamedMotorsAngularVelocityDrive(bool bEnableSwingDrive, bool bEnableTwistDrive, const TArray<FName>& BoneNames, bool bSetOtherBodiesToComplement)
{
	UPhysicsAsset* const PhysicsAsset = GetPhysicsAsset();
	if( !PhysicsAsset )
	{
		return;
	}

	for(int32 i=0; i<Constraints.Num(); i++)
	{
		FConstraintInstance* Instance = Constraints[i];
		if( BoneNames.Contains(Instance->GetChildBoneName()) )
		{
			Constraints[i]->SetAngularVelocityDriveTwistAndSwing(bEnableTwistDrive, bEnableSwingDrive);
			Constraints[i]->SetAngularVelocityDriveSLERP(bEnableTwistDrive || bEnableSwingDrive);
		}
		else if( bSetOtherBodiesToComplement )
		{
			Constraints[i]->SetAngularVelocityDriveTwistAndSwing(!bEnableTwistDrive, !bEnableSwingDrive);
			Constraints[i]->SetAngularVelocityDriveSLERP(!(bEnableTwistDrive || bEnableSwingDrive));
		}
	}
}

void USkeletalMeshComponent::SetAllMotorsAngularVelocityDrive(bool bEnableSwingDrive, bool bEnableTwistDrive, bool bSkipCustomPhysicsType)
{
	UPhysicsAsset* const PhysicsAsset = GetPhysicsAsset();
	if( !PhysicsAsset )
	{
		return;
	}

	for(int32 i=0; i<Constraints.Num(); i++)
	{
		if( bSkipCustomPhysicsType )
		{
			int32 BodyIndex = PhysicsAsset->FindBodyIndex(Constraints[i]->GetChildBoneName());
			if( BodyIndex != INDEX_NONE && PhysicsAsset->SkeletalBodySetups[BodyIndex]->PhysicsType != PhysType_Default )
			{
				continue;
			}
		}

		Constraints[i]->SetAngularVelocityDriveTwistAndSwing(bEnableTwistDrive, bEnableSwingDrive);
		Constraints[i]->SetAngularVelocityDriveSLERP(bEnableTwistDrive || bEnableSwingDrive);
	}
}

void USkeletalMeshComponent::SetConstraintProfile(FName JointName, FName ProfileName, bool bDefaultIfNotFound)
{
	UPhysicsAsset* const PhysicsAsset = GetPhysicsAsset();
	if (!PhysicsAsset)
	{
		return;
	}

	for (int32 i = 0; i < Constraints.Num(); i++)
	{
		FConstraintInstance* ConstraintInstance = Constraints[i];
		if(ConstraintInstance->JointName == JointName)
		{
			PhysicsAsset->ConstraintSetup[i]->ApplyConstraintProfile(ProfileName, *ConstraintInstance, bDefaultIfNotFound);
		}
	}
}

void USkeletalMeshComponent::SetConstraintProfileForAll(FName ProfileName, bool bDefaultIfNotFound)
{
	if(UPhysicsAsset* const PhysicsAsset = GetPhysicsAsset())
	{
		for (int32 i = 0; i < Constraints.Num(); i++)
		{
			if(FConstraintInstance* ConstraintInstance = Constraints[i])
			{
				PhysicsAsset->ConstraintSetup[i]->ApplyConstraintProfile(ProfileName, *ConstraintInstance, bDefaultIfNotFound);
			}
		}
	}
}

bool USkeletalMeshComponent::GetConstraintProfilePropertiesOrDefault(
	FConstraintProfileProperties& OutProperties, FName JointName, FName ProfileName)
{
	if (UPhysicsAsset* const PhysicsAsset = GetPhysicsAsset())
	{
		for (int32 i = 0; i < Constraints.Num(); i++)
		{
			FConstraintInstance* ConstraintInstance = Constraints[i];
			if (ConstraintInstance->JointName == JointName)
			{
				OutProperties = PhysicsAsset->ConstraintSetup[i]->GetConstraintProfilePropertiesOrDefault(ProfileName);
				return true;
			}
		}
	}
	return false;
}

void USkeletalMeshComponent::SetAllMotorsAngularDriveParams(float InSpring, float InDamping, float InForceLimit, bool bSkipCustomPhysicsType)
{
	UPhysicsAsset* const PhysicsAsset = GetPhysicsAsset();
	if( !PhysicsAsset )
	{
		return;
	}

	for(int32 i=0; i<Constraints.Num(); i++)
	{
		if( bSkipCustomPhysicsType )
		{
			int32 BodyIndex = PhysicsAsset->FindBodyIndex(Constraints[i]->GetChildBoneName());
			if( BodyIndex != INDEX_NONE && PhysicsAsset->SkeletalBodySetups[BodyIndex]->PhysicsType != PhysType_Default )
			{
				continue;
			}
		}
		Constraints[i]->SetAngularDriveParams(InSpring, InDamping, InForceLimit);
	}
}

void USkeletalMeshComponent::ResetAllBodiesSimulatePhysics()
{
	if ( !bEnablePhysicsOnDedicatedServer && IsRunningDedicatedServer() )
	{
		return;
	}

	UPhysicsAsset* const PhysicsAsset = GetPhysicsAsset();
	if( !PhysicsAsset )
	{
		return;
	}

	if(CollisionEnabledHasPhysics(GetCollisionEnabled()))
	{
		// Fix / Unfix bones
		for (int32 i = 0; i<Bodies.Num(); i++)
		{
			FBodyInstance*	BodyInst = Bodies[i];
			if (!ensure(BodyInst))
			{
				continue;
			}
			UBodySetup*	BodyInstSetup = BodyInst->GetBodySetup();

			// Set fixed on any bodies with bAlwaysFullAnimWeight set to true
			if (BodyInstSetup && BodyInstSetup->PhysicsType != PhysType_Default)
			{
				if (BodyInstSetup->PhysicsType == PhysType_Simulated)
				{
					BodyInst->SetInstanceSimulatePhysics(true, false, true);
				}
				else
				{
					BodyInst->SetInstanceSimulatePhysics(false, false, true);
				}
			}
		}
	}
}

void USkeletalMeshComponent::SetEnablePhysicsBlending(bool bNewBlendPhysics)
{
	bBlendPhysics = bNewBlendPhysics;
}

void USkeletalMeshComponent::SetPhysicsBlendWeight(float PhysicsBlendWeight)
{
	bool bShouldSimulate = PhysicsBlendWeight > 0.f;
	if (bShouldSimulate != IsSimulatingPhysics())
	{
		SetSimulatePhysics(bShouldSimulate);
	}

	// if blend weight is not 1, set manual weight
	if ( PhysicsBlendWeight < 1.f )
	{
		bBlendPhysics = false;
		SetAllBodiesPhysicsBlendWeight (PhysicsBlendWeight, true);
	}
}

void USkeletalMeshComponent::SetAllBodiesPhysicsBlendWeight(float PhysicsBlendWeight, bool bSkipCustomPhysicsType )
{
	UPhysicsAsset* const PhysicsAsset = GetPhysicsAsset();
	if( !PhysicsAsset )
	{
		return;
	}

	// Fix / Unfix bones
	for(int32 i=0; i<Bodies.Num(); i++)
	{
		FBodyInstance*	BodyInst	= Bodies[i];
		if (!ensure(BodyInst))
		{
			continue;
		}
		UBodySetup*	BodyInstSetup	= BodyInst->GetBodySetup();

		// Set fixed on any bodies with bAlwaysFullAnimWeight set to true
		if(BodyInstSetup && (!bSkipCustomPhysicsType || BodyInstSetup->PhysicsType == PhysType_Default) )
		{
			BodyInst->PhysicsBlendWeight = PhysicsBlendWeight;
		}
	}

	bBlendPhysics = false;

	UpdateEndPhysicsTickRegisteredState();
	UpdateClothTickRegisteredState();
}


void USkeletalMeshComponent::SetAllBodiesBelowPhysicsBlendWeight( const FName& InBoneName, float PhysicsBlendWeight, bool bSkipCustomPhysicsType, bool bIncludeSelf )
{
	int32 NumBodiesFound = ForEachBodyBelow(InBoneName, bIncludeSelf, bSkipCustomPhysicsType, [PhysicsBlendWeight](FBodyInstance* BI)
	{
		BI->PhysicsBlendWeight = PhysicsBlendWeight;
	});

	if (NumBodiesFound)
	{
		bBlendPhysics = false;

		UpdateEndPhysicsTickRegisteredState();
		UpdateClothTickRegisteredState();
	}
}


void USkeletalMeshComponent::AccumulateAllBodiesBelowPhysicsBlendWeight( const FName& InBoneName, float PhysicsBlendWeight, bool bSkipCustomPhysicsType )
{
	int32 NumBodiesFound = ForEachBodyBelow(InBoneName, /*bIncludeSelf=*/ true, /*bSkipCustomPhysicsType=*/ bSkipCustomPhysicsType, [PhysicsBlendWeight](FBodyInstance* BI)
	{
		BI->PhysicsBlendWeight = FMath::Min(BI->PhysicsBlendWeight + PhysicsBlendWeight, 1.f);
	});

	if (NumBodiesFound)
	{
		bBlendPhysics = false;

		UpdateEndPhysicsTickRegisteredState();
		UpdateClothTickRegisteredState();
	}
}

FConstraintInstance* USkeletalMeshComponent::FindConstraintInstance(FName ConName)
{
	UPhysicsAsset* const PhysicsAsset = GetPhysicsAsset();
	if(PhysicsAsset && PhysicsAsset->ConstraintSetup.Num() == Constraints.Num())
	{
		int32 ConIndex = PhysicsAsset->FindConstraintIndex(ConName);
		if(ConIndex != INDEX_NONE)
		{
			return Constraints[ConIndex];
		}
	}

	return NULL;
}

FConstraintInstance* USkeletalMeshComponent::GetConstraintInstanceByIndex(uint32 Index)
{
	if (Index < (uint32)Constraints.Num())
	{
		return Constraints[Index];
	}
	return nullptr;
}

void USkeletalMeshComponent::AddForceToAllBodiesBelow(FVector Force, FName BoneName, bool bAccelChange, bool bIncludeSelf)
{
	ForEachBodyBelow(BoneName, bIncludeSelf, /*bSkipCustomPhysics=*/false, [Force, bAccelChange](FBodyInstance* BI)
	{
		BI->AddForce(Force, /*bAllowSubstepping=*/ true, bAccelChange);
	});
}

void USkeletalMeshComponent::AddImpulseToAllBodiesBelow(FVector Impulse, FName BoneName, bool bVelChange, bool bIncludeSelf)
{
	ForEachBodyBelow(BoneName, bIncludeSelf,/*bSkipCustomPhysics=*/false, [Impulse, bVelChange](FBodyInstance* BI)
	{
		BI->AddImpulse(Impulse, bVelChange);
	});
}

#ifndef OLD_FORCE_UPDATE_BEHAVIOR
#define OLD_FORCE_UPDATE_BEHAVIOR 0
#endif

void USkeletalMeshComponent::OnUpdateTransform(EUpdateTransformFlags UpdateTransformFlags, ETeleportType Teleport)
{
	// We are handling the physics move below, so don't handle it at higher levels
	Super::OnUpdateTransform(UpdateTransformFlags | EUpdateTransformFlags::SkipPhysicsUpdate, Teleport);

	// Always send new transform to physics
	if(bPhysicsStateCreated && !(UpdateTransformFlags&EUpdateTransformFlags::SkipPhysicsUpdate))
	{
#if !OLD_FORCE_UPDATE_BEHAVIOR
		// Animation from the skeletal mesh happens during TG_PrePhysics.
		// Deferred kinematic updates are applied during TG_StartPhysics.
		// Propagation from the parent movement happens during TG_EndPhysics (for physics objects... could in theory be from any tick group).
		// Therefore, deferred kinematic updates are safe from animation, but not from parent movement.
		EAllowKinematicDeferral AllowDeferral = EAllowKinematicDeferral::AllowDeferral;

		if (!!(UpdateTransformFlags & EUpdateTransformFlags::PropagateFromParent))
		{
			AllowDeferral = EAllowKinematicDeferral::DisallowDeferral;
			
			// If enabled, allow deferral of PropagateFromParent updates in prephysics only.
			// Probably should rework this entire condition to be more concrete, but do not want to introduce that much risk.
			UWorld* World = GetWorld();
			if (CVarEnableKinematicDeferralPrePhysicsCondition.GetValueOnGameThread())
			{
				if (World && (World->TickGroup == ETickingGroup::TG_PrePhysics))
				{
					AllowDeferral = EAllowKinematicDeferral::AllowDeferral;
				}
			}

			if(GEnableKinematicDeferralStartPhysicsCondition)
			{
				if (World && (World->TickGroup == ETickingGroup::TG_StartPhysics))
				{
					AllowDeferral = EAllowKinematicDeferral::AllowDeferral;
				}
			}
		}

		UpdateKinematicBonesToAnim(GetComponentSpaceTransforms(), Teleport, false, AllowDeferral);
#else
		UpdateKinematicBonesToAnim(GetComponentSpaceTransforms(), ETeleportType::TeleportPhysics, false);
#endif
	}

	// Pass teleports on to anything in the animation tree that might be interested (e.g. AnimDynamics, RigidBody Node, etc.)
	if(Teleport != ETeleportType::None)
	{
		ResetAnimInstanceDynamics(Teleport);
	}

	// Mark the cloth simulation transform update as pending (the actual update will happen in the cloth thread if the simulation is running)
	UpdateClothTransform(CVarClothPhysicsOnUpdateTransformTeleport.GetValueOnGameThread() ? Teleport : ETeleportType::None);

	if (CVarUpdateRigidBodyScaling.GetValueOnGameThread())
	{
		//Updates the scale of the rigid bodies for physics. 
		UpdateRigidBodyScaling(Teleport);
	}
}

bool USkeletalMeshComponent::UpdateOverlapsImpl(const TOverlapArrayView* PendingOverlaps, bool bDoNotifies, const TOverlapArrayView* OverlapsAtEndLocation)
{
	// Parent class (USkinnedMeshComponent) routes only to children, but we really do want to test our own bodies for overlaps.
	return UPrimitiveComponent::UpdateOverlapsImpl(PendingOverlaps, bDoNotifies, OverlapsAtEndLocation);
}

bool USkeletalMeshComponent::ShouldCreatePhysicsState() const
{
	bool bShouldCreatePhysicsState = Super::ShouldCreatePhysicsState();
	bShouldCreatePhysicsState &= (LeaderPoseComponent.IsValid() == false);
	
	return bShouldCreatePhysicsState;
}

bool USkeletalMeshComponent::HasValidPhysicsState() const
{
	return !Bodies.IsEmpty();
}

void USkeletalMeshComponent::OnCreatePhysicsState()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(USkeletalMeshComponent::OnCreatePhysicsState);

	// Init physics
	if (bEnablePerPolyCollision == false)
	{
		InitArticulated(GetWorld()->GetPhysicsScene());
		USceneComponent::OnCreatePhysicsState(); // Need to route CreatePhysicsState, skip PrimitiveComponent
#if UE_ENABLE_DEBUG_DRAWING
		SendRenderDebugPhysics();
#endif
		// Update per-frame physics quantization registration
		UE::PhysicsReplicationQuantization::NotifyComponentPhysicsState(this, /*bRegister*/true);
	}
	else
	{
		CreateBodySetup();
		BodySetup->CreatePhysicsMeshes();
		Super::OnCreatePhysicsState();	//If we're doing per poly we'll use the body instance of the primitive component
	}

	// Notify physics created
	OnSkelMeshPhysicsCreated.Broadcast();
}


void USkeletalMeshComponent::OnDestroyPhysicsState()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(USkeletalMeshComponent::OnDestroyPhysicsState);

	// Update per-frame physics quantization registration
	UE::PhysicsReplicationQuantization::NotifyComponentPhysicsState(this, /*bRegister*/false);

	DestroyPhysicsStateInternal();

	Super::OnDestroyPhysicsState();
}

void USkeletalMeshComponent::DestroyPhysicsStateInternal()
{
	check(IsInGameThread());

	if (bEnablePerPolyCollision == false)
	{
		UnWeldFromParent();
		UnWeldChildren();
		TermArticulated();
	}
}

#if 0 && !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
#define DEBUGBROKENCONSTRAINTUPDATE(x) { ##x }
#else
#define DEBUGBROKENCONSTRAINTUPDATE(x)
#endif

#if UE_ENABLE_DEBUG_DRAWING
void USkeletalMeshComponent::SendRenderDebugPhysics(FPrimitiveSceneProxy* OverrideSceneProxy)
{
	FPrimitiveSceneProxy* UseSceneProxy = OverrideSceneProxy ? OverrideSceneProxy : SceneProxy;
	if (UseSceneProxy)
	{
		TArray<FPrimitiveSceneProxy::FDebugMassData> DebugMassData;
		DebugMassData.Reserve(Bodies.Num());
		
		for (FBodyInstance* BI : Bodies)
		{
			if (BI && BI->IsValidBodyInstance())
			{
				const int32 BoneIndex = BI->InstanceBoneIndex;
				if (BoneIndex >= GetComponentSpaceTransforms().Num())
				{
					UE_LOGF(LogSkeletalMesh, Log, "SkeletalMeshComponent : (%d) Bone index out of bounds", BoneIndex);
					continue;
				}
				DebugMassData.AddDefaulted();
				FPrimitiveSceneProxy::FDebugMassData& MassData = DebugMassData.Last();
				const FTransform MassToWorld = BI->GetMassSpaceToWorldSpace();
				const FTransform& BoneTM = GetComponentSpaceTransforms()[BoneIndex];
				const FTransform BoneToWorld = BoneTM * GetComponentTransform();

				MassData.LocalCenterOfMass = BoneToWorld.InverseTransformPosition(MassToWorld.GetLocation());
				MassData.LocalTensorOrientation = MassToWorld.GetRotation() * BoneToWorld.GetRotation().Inverse();
				MassData.MassSpaceInertiaTensor = BI->GetBodyInertiaTensor();
				MassData.BoneIndex = BoneIndex;
			}
		}

		ENQUEUE_RENDER_COMMAND(SkeletalMesh_SendRenderDebugPhysics)(UE::RenderCommandPipe::SkeletalMesh,
			[UseSceneProxy, DebugMassData]
			{
				UseSceneProxy->SetDebugMassData(DebugMassData);
			}
		);
		
	}
}
#endif

void USkeletalMeshComponent::UpdateMeshForBrokenConstraints()
{
	UPhysicsAsset * const PhysicsAsset = GetPhysicsAsset();
	// Needs to have a SkeletalMesh, and PhysicsAsset.
	if( !GetSkeletalMeshAsset() || !PhysicsAsset )
	{
		return;
	}

	DEBUGBROKENCONSTRAINTUPDATE(UE_LOGF(LogSkeletalMesh, Log, "%3.3f UpdateMeshForBrokenConstraints", GetWorld()->GetTimeSeconds());)

	// Iterate through list of constraints in the physics asset
	for(int32 ConstraintInstIndex = 0; ConstraintInstIndex < Constraints.Num(); ConstraintInstIndex++)
	{
		// See if we can find a constraint that has been terminated (broken)
		FConstraintInstance* ConstraintInst = Constraints[ConstraintInstIndex];
		if( ConstraintInst && ConstraintInst->IsTerminated() )
		{
			// Get the associated joint bone index.
			int32 JointBoneIndex = GetBoneIndex(ConstraintInst->GetChildBoneName());
			if( JointBoneIndex == INDEX_NONE )
			{
				continue;
			}

			DEBUGBROKENCONSTRAINTUPDATE(UE_LOGF(LogSkeletalMesh, Log, "  Found Broken Constraint: (%d) %ls", JointBoneIndex, *PhysicsAsset->ConstraintSetup(ConstraintInstIndex)->JointName.ToString());)

			// Get child bodies of this joint
			for(int32 BodySetupIndex = 0; BodySetupIndex < PhysicsAsset->SkeletalBodySetups.Num(); BodySetupIndex++)
			{
				UBodySetup* PhysicsAssetBodySetup = PhysicsAsset->SkeletalBodySetups[BodySetupIndex];
				int32 BoneIndex = GetBoneIndex(PhysicsAssetBodySetup->BoneName);
				if( BoneIndex != INDEX_NONE && 
					(BoneIndex == JointBoneIndex || GetSkeletalMeshAsset()->GetRefSkeleton().BoneIsChildOf(BoneIndex, JointBoneIndex)) )
				{
					DEBUGBROKENCONSTRAINTUPDATE(UE_LOGF(LogSkeletalMesh, Log, "    Found Child Bone: (%d) %ls", BoneIndex, *PhysicsAssetBodySetup->BoneName.ToString());)

					FBodyInstance* ChildBodyInst = Bodies[BodySetupIndex];
					if( ChildBodyInst )
					{
						// Unfix Body so, it is purely physical, not kinematic.
						if( !ChildBodyInst->IsInstanceSimulatingPhysics() )
						{
							DEBUGBROKENCONSTRAINTUPDATE(UE_LOGF(LogSkeletalMesh, Log, "      Unfixing body.");)
							ChildBodyInst->SetInstanceSimulatePhysics(true, false, true);
						}
					}

					FConstraintInstance* ChildConstraintInst = FindConstraintInstance(PhysicsAssetBodySetup->BoneName);
					if( ChildConstraintInst )
					{
						if( ChildConstraintInst->IsLinearPositionDriveEnabled() )
						{
							DEBUGBROKENCONSTRAINTUPDATE(UE_LOGF(LogSkeletalMesh, Log, "      Turning off LinearPositionDrive.");)
							ChildConstraintInst->SetLinearPositionDrive(false, false, false);
						}
						if( ChildConstraintInst->IsLinearVelocityDriveEnabled() )
						{
							DEBUGBROKENCONSTRAINTUPDATE(UE_LOGF(LogSkeletalMesh, Log, "      Turning off LinearVelocityDrive.");)
							ChildConstraintInst->SetLinearVelocityDrive(false, false, false);
						}
						if( ChildConstraintInst->IsAngularOrientationDriveEnabled() )
						{
							DEBUGBROKENCONSTRAINTUPDATE(UE_LOGF(LogSkeletalMesh, Log, "      Turning off AngularPositionDrive.");)
							ChildConstraintInst->SetOrientationDriveTwistAndSwing(false, false);
						}
						if( ChildConstraintInst->IsAngularVelocityDriveEnabled() )
						{
							DEBUGBROKENCONSTRAINTUPDATE(UE_LOGF(LogSkeletalMesh, Log, "      Turning off AngularVelocityDrive.");)
							ChildConstraintInst->SetAngularVelocityDriveTwistAndSwing(false, false);
						}
					}
				}
			}
		}
	}
}


int32 USkeletalMeshComponent::FindConstraintIndex( FName ConstraintName )
{
	UPhysicsAsset * const PhysicsAsset = GetPhysicsAsset();
	return PhysicsAsset ? PhysicsAsset->FindConstraintIndex(ConstraintName) : INDEX_NONE;
}


FName USkeletalMeshComponent::FindConstraintBoneName( int32 ConstraintIndex )
{
	UPhysicsAsset * const PhysicsAsset = GetPhysicsAsset();
	return PhysicsAsset ? PhysicsAsset->FindConstraintBoneName(ConstraintIndex) : NAME_None;
}

bool USkeletalMeshComponent::IsSimulatingPhysics(FName BoneName) const
{
	// If no bone name is specified, then we respond referring to the component.
	// If the component is not set to follow physics, then the component is not controlled by simulation.
	if (BoneName == NAME_None && 
		PhysicsTransformUpdateMode == EPhysicsTransformUpdateMode::ComponentTransformIsKinematic)
	{
		return false;
	}

	// We respond based on either the body (if a bone is specified), or the root body (if no bone is
	// specified, and the component is controlled by simulation).
	FBodyInstance* BI = GetBodyInstance(BoneName);
	if (BI)
	{
		return BI->IsInstanceSimulatingPhysics();
	}
	return false;
}

FBodyInstance* USkeletalMeshComponent::GetBodyInstance(FName BoneName, bool, int32) const
{
	UPhysicsAsset * const PhysicsAsset = GetPhysicsAsset();
	FBodyInstance* BodyInst = NULL;

	if(PhysicsAsset != NULL)
	{
		// A name of NAME_None indicates 'root body'
		if(BoneName == NAME_None)
		{
			if (Bodies.IsValidIndex(RootBodyData.BodyIndex))
			{
				BodyInst = Bodies[RootBodyData.BodyIndex];
			}
		}
		// otherwise, look for the body
		else
		{
			int32 BodyIndex = PhysicsAsset->FindBodyIndex(BoneName);
			if(Bodies.IsValidIndex(BodyIndex))
			{
				BodyInst = Bodies[BodyIndex];
			}
		}

	}

	return BodyInst;
}

void USkeletalMeshComponent::GetWeldedBodies(TArray<FBodyInstance*> & OutWeldedBodies, TArray<FName> & OutLabels, bool bIncludingAutoWeld)
{
	UPhysicsAsset* PhysicsAsset = GetPhysicsAsset();

	for (int32 BodyIdx = 0; BodyIdx < Bodies.Num(); ++BodyIdx)
	{
		FBodyInstance* BI = Bodies[BodyIdx];
		if (BI && (BI->WeldParent != nullptr || (bIncludingAutoWeld && BI->bAutoWeld)))
		{
			OutWeldedBodies.Add(BI);
			if (PhysicsAsset)
			{
				if (UBodySetup * PhysicsAssetBodySetup = PhysicsAsset->SkeletalBodySetups[BodyIdx])
				{
					OutLabels.Add(PhysicsAssetBodySetup->BoneName);
				}
				else
				{
					OutLabels.Add(NAME_None);
				}
			}
			else
			{
				OutLabels.Add(NAME_None);
			}

			for (USceneComponent * Child : GetAttachChildren())
			{
				if (UPrimitiveComponent * PrimChild = Cast<UPrimitiveComponent>(Child))
				{
					PrimChild->GetWeldedBodies(OutWeldedBodies, OutLabels, bIncludingAutoWeld);
				}
			}
		}
	}
}

int32 USkeletalMeshComponent::ForEachBodyBelow(FName BoneName, bool bIncludeSelf, bool bSkipCustomType, TFunctionRef<void(FBodyInstance*)> Func)
{
	if (BoneName == NAME_None && bIncludeSelf && !bSkipCustomType)
	{
		for (FBodyInstance* BI : Bodies)	//we want all bodies so just iterate the regular array
		{
			Func(BI);
		}

		return Bodies.Num();
	}
	else
	{
		UPhysicsAsset* const PhysicsAsset = GetPhysicsAsset();
		if (!PhysicsAsset || !GetSkeletalMeshAsset())
		{
			return 0;
		}

		// if physics state is invalid - i.e. collision is disabled - or it does not have a valid bodies, this will crash right away
		if (!IsPhysicsStateCreated() || !bHasValidBodies)
		{
			FMessageLog("PIE").Warning(LOCTEXT("InvalidBodies", "Invalid Bodies : Make sure collision is enabled or root bone has body in PhysicsAsset."));
			return 0;
		}

		TArray<int32> BodyIndices;
		BodyIndices.Reserve(Bodies.Num());
		PhysicsAsset->GetBodyIndicesBelow(BodyIndices, BoneName, GetSkeletalMeshAsset(), bIncludeSelf);

		int32 NumBodiesFound = 0;
		for (int32 BodyIdx : BodyIndices)
		{
			// GetBodyIndicesBelow does not take into account the actual Bodies we've created.
			if (!Bodies.IsValidIndex(BodyIdx))
			{
				UE_LOGF(LogSkeletalMesh, Warning, "%s: assumed we had BodyIdx %d but we only have %d Bodies", __FUNCTION__, BodyIdx, Bodies.Num());
				break;
			}

			FBodyInstance* BI = Bodies[BodyIdx];
			if (bSkipCustomType)
			{
				if (UBodySetup* PhysAssetBodySetup = PhysicsAsset->SkeletalBodySetups[BodyIdx])
				{
					if (PhysAssetBodySetup->PhysicsType != EPhysicsType::PhysType_Default)
					{
						continue;
					}
				}
			}

			++NumBodiesFound;
			Func(BI);
		}

		return NumBodiesFound;
	}
}

void USkeletalMeshComponent::SetNotifyRigidBodyCollision(bool bNewNotifyRigidBodyCollision)
{
	for(FBodyInstance* BI : Bodies)
	{
		BI->SetInstanceNotifyRBCollision(bNewNotifyRigidBodyCollision);
	}

	if(Bodies.Num() > 0)
	{
		OnComponentCollisionSettingsChanged();
	}
}

void USkeletalMeshComponent::SetBodyNotifyRigidBodyCollision(bool bNewNotifyRigidBodyCollision, FName BoneName /* = NAME_None */)
{
	if(FBodyInstance* BI = GetBodyInstance(BoneName))
	{
		BI->SetInstanceNotifyRBCollision(bNewNotifyRigidBodyCollision);

		OnComponentCollisionSettingsChanged();
	}
}

void USkeletalMeshComponent::SetNotifyRigidBodyCollisionBelow(bool bNewNotifyRigidBodyCollision, FName BoneName, bool bIncludeSelf)
{
	const int32 NumBodiesFound = ForEachBodyBelow(BoneName, bIncludeSelf, /*bSkipCustomType=*/false, [bNewNotifyRigidBodyCollision](FBodyInstance* BI)
	{
		BI->SetInstanceNotifyRBCollision(bNewNotifyRigidBodyCollision);
	});
	
	if(NumBodiesFound > 0)
	{
		OnComponentCollisionSettingsChanged();
	}
}

FConstraintInstanceAccessor USkeletalMeshComponent::GetConstraintByName(FName ConstraintName, bool bIncludesTerminated)
{
	int32 ConstraintIndex = FindConstraintIndex(ConstraintName);
	if (ConstraintIndex == INDEX_NONE || ConstraintIndex >= Constraints.Num())
	{
		return FConstraintInstanceAccessor();
	}

	if (FConstraintInstance* Constraint = Constraints[ConstraintIndex])
	{
		if (bIncludesTerminated || !Constraint->IsTerminated())
		{
			return FConstraintInstanceAccessor(this, ConstraintIndex);
		}
	}
	return FConstraintInstanceAccessor();
}

void USkeletalMeshComponent::GetConstraints(bool bIncludesTerminated, TArray<FConstraintInstanceAccessor>& OutConstraints)
{
	if (UPhysicsAsset* const PhysicsAsset = GetPhysicsAsset())
	{
		for (int32 i = 0; i < Constraints.Num(); i++)
		{
			if (FConstraintInstance* ConstraintInstance = Constraints[i])
			{
				if (bIncludesTerminated || !ConstraintInstance->IsTerminated())
				{
					OutConstraints.Add(FConstraintInstanceAccessor(this, i));
				}
			}
		}
	}
}

void  USkeletalMeshComponent::GetConstraintsFromBody(FName BodyName, bool bParentConstraints, bool bChildConstraints, bool bIncludesTerminated, TArray<FConstraintInstanceAccessor>& OutConstraints)
{
	if (UPhysicsAsset* const PhysicsAsset = GetPhysicsAsset())
	{
		for (int32 i = 0; i < Constraints.Num(); i++)
		{
			if (FConstraintInstance* ConstraintInstance = Constraints[i])
			{
				if (bIncludesTerminated || !ConstraintInstance->IsTerminated())
				{
					if (bParentConstraints && ConstraintInstance->GetChildBoneName() == BodyName)
					{
						OutConstraints.Add(FConstraintInstanceAccessor(this, i));
					}
					if (bChildConstraints && ConstraintInstance->GetParentBoneName() == BodyName)
					{
						OutConstraints.Add(FConstraintInstanceAccessor(this, i));
					}
				}
			}
		}
	}
}

void USkeletalMeshComponent::BreakConstraint(FVector Impulse, FVector HitLocation, FName InBoneName)
{
	// you can enable/disable the instanced weights by calling
	int32 ConstraintIndex = FindConstraintIndex(InBoneName);
	if( ConstraintIndex == INDEX_NONE || ConstraintIndex >= Constraints.Num() )
	{
		return;
	}

	FConstraintInstance* Constraint = Constraints[ConstraintIndex];
	// If already broken, our job has already been done. Bail!
	if( Constraint->IsTerminated() )
	{
		return;
	}

	UPhysicsAsset * const PhysicsAsset = GetPhysicsAsset();

	// Figure out if Body is fixed or not
	FBodyInstance* Body = GetBodyInstance(Constraint->GetChildBoneName());

	if( Body != NULL && !Body->IsInstanceSimulatingPhysics() )
	{
		// Unfix body so it can be broken.
		Body->SetInstanceSimulatePhysics(true, false, true);
	}

	// Break Constraint
	Constraint->TermConstraint();
	// Make sure child bodies and constraints are released and turned to physics.
	UpdateMeshForBrokenConstraints();
	// Add impulse to broken limb
	AddImpulseAtLocation(Impulse, HitLocation, InBoneName);
}


void USkeletalMeshComponent::SetAngularLimits(FName InBoneName, float Swing1LimitAngle, float TwistLimitAngle, float Swing2LimitAngle)
{
	int32 ConstraintIndex = FindConstraintIndex(InBoneName);
	if (ConstraintIndex == INDEX_NONE || ConstraintIndex >= Constraints.Num())
	{
		return;
	}

	FConstraintInstance* Constraint = Constraints[ConstraintIndex];
	// If already broken, our job has already been done. Bail!
	if (Constraint->IsTerminated())
	{
		return;
	}

	UPhysicsAsset * const PhysicsAsset = GetPhysicsAsset();

	// Figure out if Body is fixed or not
	FBodyInstance* Body = GetBodyInstance(Constraint->GetChildBoneName());

	if (Body != NULL && Body->IsInstanceSimulatingPhysics())
	{
		// Unfix body so it can be broken.
		Body->SetInstanceSimulatePhysics(true, false, true);
	}

	// update limits
	Constraint->SetAngularSwing1Limit(Swing1LimitAngle == 0 ? ACM_Locked : (Swing1LimitAngle >= 180) ? ACM_Free : ACM_Limited, Swing1LimitAngle);
	Constraint->SetAngularTwistLimit(TwistLimitAngle == 0 ? ACM_Locked : (TwistLimitAngle >= 180) ? ACM_Free : ACM_Limited, TwistLimitAngle);
	Constraint->SetAngularSwing2Limit(Swing2LimitAngle == 0 ? ACM_Locked : (Swing2LimitAngle >= 180) ? ACM_Free : ACM_Limited, Swing2LimitAngle);
}


void USkeletalMeshComponent::GetCurrentJointAngles(FName InBoneName, float &Swing1Angle, float &TwistAngle, float &Swing2Angle)
{
	int32 ConstraintIndex = FindConstraintIndex(InBoneName);
	if (ConstraintIndex == INDEX_NONE || ConstraintIndex >= Constraints.Num())
	{
		return;
	}

	FConstraintInstance* Constraint = Constraints[ConstraintIndex];
	
	Swing1Angle = FMath::RadiansToDegrees(Constraint->GetCurrentSwing1());
	Swing2Angle = FMath::RadiansToDegrees(Constraint->GetCurrentSwing2());
	TwistAngle = FMath::RadiansToDegrees(Constraint->GetCurrentTwist());
}


void USkeletalMeshComponent::SetPhysicsAsset(UPhysicsAsset* InPhysicsAsset, bool bForceReInit)
{
	// If this is different from what we have now, or we should have an instance but for whatever reason it failed last time, teardown/recreate now.
	if(bForceReInit || InPhysicsAsset != GetPhysicsAsset())
	{
		// SkelComp had a physics instance, then terminate it.
		TermArticulated();

		// Need to update scene proxy, because it keeps a ref to the PhysicsAsset.
		Super::SetPhysicsAsset(InPhysicsAsset, bForceReInit);
		MarkRenderStateDirty();

		// Update bHasValidBodies flag
		UpdateHasValidBodies();

		// Component should be re-attached here, so create physics.
		if(GetSkeletalMeshAsset())
		{
			// Because we don't know what bones the new PhysicsAsset might want, we have to force an update to _all_ bones in the skeleton.
			RequiredBones.Reset(GetSkeletalMeshAsset()->GetRefSkeleton().GetNum());
			RequiredBones.AddUninitialized(GetSkeletalMeshAsset()->GetRefSkeleton().GetNum() );
			for(int32 i=0; i< GetSkeletalMeshAsset()->GetRefSkeleton().GetNum(); i++)
			{
				RequiredBones[i] = (FBoneIndexType)i;
			}
			RefreshBoneTransforms();

			// Initialize new Physics Asset
			UWorld* World = GetWorld();
			if(World && World->GetPhysicsScene() != nullptr)
			{
			//	UE_LOGF(LogSkeletalMesh, Warning, "Creating Physics State (%ls : %ls)", *GetNameSafe(GetOuter()),  *GetName());			
				InitArticulated(World->GetPhysicsScene());
			}
		}
		else
		{
			// If PhysicsAsset hasn't been instanced yet, just update the template.
			Super::SetPhysicsAsset(InPhysicsAsset, bForceReInit);

			// Update bHasValidBodies flag
			UpdateHasValidBodies();
		}

		// Indicate that 'required bones' array will need to be recalculated.
		bRequiredBonesUpToDate = false;

#if UE_ENABLE_DEBUG_DRAWING
		SendRenderDebugPhysics();
#endif
	}
}


void USkeletalMeshComponent::UpdateHasValidBodies()
{
	// First clear out old data
	bHasValidBodies = false;

	const UPhysicsAsset* const PhysicsAsset = GetPhysicsAsset();

	// If we have a physics asset..
	if(PhysicsAsset != NULL)
	{
		// For each body in physics asset..
		for( int32 BodyIndex = 0; BodyIndex < PhysicsAsset->SkeletalBodySetups.Num(); BodyIndex++ )
		{
			int32 BoneIndex = INDEX_NONE;
			// .. find the matching graphics bone index
			if (TObjectPtr<USkeletalBodySetup> SkeletalBodySetup = PhysicsAsset->SkeletalBodySetups[BodyIndex])
			{
				BoneIndex = GetBoneIndex(SkeletalBodySetup->BoneName);
			}

			// If we found a valid graphics bone, set the 'valid' flag
			if(BoneIndex != INDEX_NONE)
			{
				bHasValidBodies = true;
				break;
			}
		}
	}
}

void USkeletalMeshComponent::UpdateBoneBodyMapping()
{
	if (Bodies.Num() > 0)	//If using per poly then there's no bodies to update indices on
	{
		// If we have a physics asset..
		if (const UPhysicsAsset* const PhysicsAsset = GetPhysicsAsset())
		{
			bool bNeedsReInit = false;

			// For each body in physics asset..
			for (int32 BodyIndex = 0; BodyIndex < PhysicsAsset->SkeletalBodySetups.Num(); BodyIndex++)
			{
				// .. find the matching graphics bone index
				int32 BoneIndex = GetBoneIndex(PhysicsAsset->SkeletalBodySetups[BodyIndex]->BoneName);
				FBodyInstance* Inst = Bodies[BodyIndex];
				check(Inst);

				// Make sure physics state matches presence of bone
				bool bHasValidBone = (BoneIndex != INDEX_NONE);
				if (bHasValidBone != Inst->IsValidBodyInstance())
				{
					// If not, we need to recreate physics asset to clean up bodies or create new ones
					bNeedsReInit = true;
				}

				Inst->InstanceBoneIndex = BoneIndex;
			}

			// If the set of bodies needs to change, we recreate physics asset
			if (bNeedsReInit)
			{
				RecreatePhysicsState();
			}
		}
	}
}

void USkeletalMeshComponent::UpdatePhysicsToRBChannels()
{
	// Iterate over each bone/body.
	for (int32 i = 0; i < Bodies.Num(); i++)
	{
		FBodyInstance* BI = Bodies[i];
		check(BI);
		BI->UpdatePhysicsFilterData();
	}
}

template<bool bCachedMatrices>
FVector GetTypedSkinnedVertexPositionWithCloth(USkeletalMeshComponent* Component, int32 VertexIndex, const FSkeletalMeshLODRenderData& LODData, const FSkinWeightVertexBuffer& SkinWeightBuffer, TArray<FMatrix44f>& CachedRefToLocals)
{
	// Find the chunk and vertex within that chunk, and skinning type, for this vertex.
	int32 SectionIndex;
	int32 VertIndexInChunk;
	LODData.GetSectionFromVertexIndex(VertexIndex, SectionIndex, VertIndexInChunk);
	const FSkelMeshRenderSection& Section = LODData.RenderSections[SectionIndex];

	// only if this component has clothing and is showing simulated results	
	if (Component->GetSkeletalMeshAsset() &&
		Component->GetSkeletalMeshAsset()->GetMeshClothingAssets().Num() > 0 &&
		!Component->bDisableClothSimulation &&
		Component->ClothBlendWeight > 0.0f // if cloth blend weight is 0.0, only showing skinned vertices regardless of simulation positions
		)
	{
		bool bClothVertex = false;
		FGuid ClothAssetGuid;

		// if this section corresponds to a cloth section, returns corresponding cloth section's info instead
		// if this chunk has cloth data
		if (Section.HasClothingData())
		{
			bClothVertex = true;
			ClothAssetGuid = Section.ClothingData.AssetGuid;
		}

		if (bClothVertex)
		{
			FVector SimulatedPos;
			if (Component->GetClothSimulatedPosition_GameThread(ClothAssetGuid, VertIndexInChunk, SimulatedPos))
			{
				// a simulated position is in world space and convert this to local space
				// because SkinnedMeshComponent::GetSkinnedVertexPosition() returns the position in local space
				SimulatedPos = Component->GetComponentTransform().InverseTransformPosition(SimulatedPos);

				// if blend weight is 1.0, doesn't need to blend with a skinned position
				if (Component->ClothBlendWeight < 1.0f)
				{
					// blend with a skinned position
					FVector SkinnedPos = (FVector)GetTypedSkinnedVertexPosition<bCachedMatrices>(Component, Section, LODData.StaticVertexBuffers.PositionVertexBuffer, SkinWeightBuffer, VertIndexInChunk, CachedRefToLocals);

					SimulatedPos = SimulatedPos*Component->ClothBlendWeight + SkinnedPos*(1.0f - Component->ClothBlendWeight);
				}
				return SimulatedPos;
			}
		}
	}

	return (FVector)GetTypedSkinnedVertexPosition<bCachedMatrices>(Component, Section, LODData.StaticVertexBuffers.PositionVertexBuffer, SkinWeightBuffer, VertIndexInChunk, CachedRefToLocals);
}

FVector3f USkeletalMeshComponent::GetSkinnedVertexPosition(USkeletalMeshComponent* Component, int32 VertexIndex, const FSkeletalMeshLODRenderData& LODData, const FSkinWeightVertexBuffer& SkinWeightBuffer)
{
	TArray<FMatrix44f> Dummy;
	return (FVector3f)GetTypedSkinnedVertexPositionWithCloth<false>(Component, VertexIndex, LODData, SkinWeightBuffer, Dummy);
}

FVector3f USkeletalMeshComponent::GetSkinnedVertexPosition(USkeletalMeshComponent* Component, int32 VertexIndex, const FSkeletalMeshLODRenderData& LODData, const FSkinWeightVertexBuffer& SkinWeightBuffer, TArray<FMatrix44f>& CachedRefToLocals)
{
	return (FVector3f)GetTypedSkinnedVertexPositionWithCloth<true>(Component, VertexIndex, LODData, SkinWeightBuffer, CachedRefToLocals);
}

void USkeletalMeshComponent::ComputeSkinnedPositions(USkeletalMeshComponent* Component, TArray<FVector3f> & OutPositions, TArray<FMatrix44f>& CachedRefToLocals, const FSkeletalMeshLODRenderData& LODData, const FSkinWeightVertexBuffer& SkinWeightBuffer)
{
	// Fail if no mesh
	if (!Component->GetSkeletalMeshAsset())
	{
		return;
	}

	if (Component->GetSkeletalMeshAsset()->GetMeshClothingAssets().Num() > 0 &&
		!Component->bDisableClothSimulation &&
		Component->ClothBlendWeight > 0.0f // if cloth blend weight is 0.0, only showing skinned vertices regardless of simulation positions
		)
	{
		// Fail if cloth data not available
		const TMap<int32, FClothSimulData>& ClothData = Component->GetCurrentClothingData_GameThread();
		if (ClothData.Num() == 0)
		{
			return;
		}

		OutPositions.Empty();
		OutPositions.AddUninitialized(LODData.GetNumVertices());

		//update positions
		for (int32 SectionIdx = 0; SectionIdx < LODData.RenderSections.Num(); ++SectionIdx)
		{
			const FSkelMeshRenderSection& Section = LODData.RenderSections[SectionIdx];

			bool bClothVertex = false;
			int32 ClothAssetIndex = -1;
			FGuid ClothAssetGuid;

			// if this section corresponds to a cloth section, returns corresponding cloth section's info instead
			// if this chunk has cloth data
			if (Section.HasClothingData())
			{
				bClothVertex = true;
				ClothAssetIndex = Section.CorrespondClothAssetIndex;
				ClothAssetGuid = Section.ClothingData.AssetGuid;
			}

			if (bClothVertex)
			{
				int32 AssetIndex = Component->GetSkeletalMeshAsset()->GetClothingAssetIndex(ClothAssetGuid);
				if (AssetIndex != INDEX_NONE)
				{
					const FClothSimulData* ActorData = ClothData.Find(AssetIndex);

					if (ActorData)
					{
						const uint32 SoftOffset = Section.GetVertexBufferIndex();
						const uint32 NumSoftVerts = FMath::Min(Section.GetNumVertices(), ActorData->Positions.Num());

						// if blend weight is 1.0, doesn't need to blend with a skinned position
						if (Component->ClothBlendWeight < 1.0f)
						{
							for (uint32 SoftIdx = 0; SoftIdx < NumSoftVerts; ++SoftIdx)
							{
								FVector3f SimulatedPos = ActorData->Positions[SoftIdx];

								FVector3f SkinnedPosition = GetTypedSkinnedVertexPosition<true>(Component, Section, LODData.StaticVertexBuffers.PositionVertexBuffer, SkinWeightBuffer, SoftIdx, CachedRefToLocals);

								OutPositions[SoftOffset + SoftIdx] = SimulatedPos*Component->ClothBlendWeight + SkinnedPosition*(1.0f - Component->ClothBlendWeight);
							}
						}
						else
						{
							for (uint32 SoftIdx = 0; SoftIdx < NumSoftVerts; ++SoftIdx)
							{
								OutPositions[SoftOffset + SoftIdx] = ActorData->Positions[SoftIdx];
							}
						}

						return;
					}
				}
			}

			//fall back to just regular skinning.
			const uint32 SoftOffset = Section.GetVertexBufferIndex();
			const uint32 NumSoftVerts = Section.GetNumVertices();
			for (uint32 SoftIdx = 0; SoftIdx < NumSoftVerts; ++SoftIdx)
			{
				FVector3f SkinnedPosition = GetTypedSkinnedVertexPosition<true>(Component, Section, LODData.StaticVertexBuffers.PositionVertexBuffer, SkinWeightBuffer, SoftIdx, CachedRefToLocals);
				OutPositions[SoftOffset + SoftIdx] = SkinnedPosition;
			}
		}
	}
	else
	{
		OutPositions.Empty();
		OutPositions.AddUninitialized(LODData.GetNumVertices());

		for (int32 SectionIdx = 0; SectionIdx < LODData.RenderSections.Num(); ++SectionIdx)
		{
			const FSkelMeshRenderSection& Section = LODData.RenderSections[SectionIdx];

			const uint32 SoftOffset = Section.GetVertexBufferIndex();
			const uint32 NumSoftVerts = Section.GetNumVertices();
			for (uint32 SoftIdx = 0; SoftIdx < NumSoftVerts; ++SoftIdx)
			{
				FVector3f SkinnedPosition = GetTypedSkinnedVertexPosition<true>(Component, Section, LODData.StaticVertexBuffers.PositionVertexBuffer, SkinWeightBuffer, SoftIdx, CachedRefToLocals);
				OutPositions[SoftOffset + SoftIdx] = SkinnedPosition;
			}
		}
	}
}

void USkeletalMeshComponent::GetSkinnedTangentBasis(USkeletalMeshComponent* Component, int32 VertexIndex, const FSkeletalMeshLODRenderData& LODData, const FSkinWeightVertexBuffer& SkinWeightBuffer, TArray<FMatrix44f>& CachedRefToLocals, FVector3f& OutTangentX, FVector3f& OutTangentY, FVector3f& OutTangentZ)
{
	int32 SectionIndex;
	int32 VertIndexInChunk;
	LODData.GetSectionFromVertexIndex(VertexIndex, SectionIndex, VertIndexInChunk);

	check(SectionIndex < LODData.RenderSections.Num());
	const FSkelMeshRenderSection& Section = LODData.RenderSections[SectionIndex];

	return GetTypedSkinnedTangentBasis(Component, Section, LODData.StaticVertexBuffers, SkinWeightBuffer, VertIndexInChunk, CachedRefToLocals, OutTangentX, OutTangentY, OutTangentZ);
}

void USkeletalMeshComponent::ComputeSkinnedTangentBasis(USkeletalMeshComponent* Component, TArray<FVector3f>& OutTangenXYZ, TArray<FMatrix44f>& CachedRefToLocals, const FSkeletalMeshLODRenderData& LODData, const FSkinWeightVertexBuffer& SkinWeightBuffer)
{
	// Fail if no mesh
	if (!Component->GetSkeletalMeshAsset())
	{
		return;
	}

	OutTangenXYZ.Empty();
	OutTangenXYZ.AddUninitialized(LODData.GetNumVertices() * 3);

	for (int32 SectionIdx = 0; SectionIdx < LODData.RenderSections.Num(); ++SectionIdx)
	{
		const FSkelMeshRenderSection& Section = LODData.RenderSections[SectionIdx];

		const uint32 SoftOffset = Section.GetVertexBufferIndex();
		const uint32 NumSoftVerts = Section.GetNumVertices();
		for (uint32 SoftIdx = 0; SoftIdx < NumSoftVerts; ++SoftIdx)
		{
			const uint32 TangentOffset = (SoftOffset + SoftIdx) * 3;
			GetTypedSkinnedTangentBasis(Component, Section, LODData.StaticVertexBuffers, SkinWeightBuffer, SoftIdx, CachedRefToLocals, OutTangenXYZ[TangentOffset + 0], OutTangenXYZ[TangentOffset + 1], OutTangenXYZ[TangentOffset + 2]);
		}
	}
}

void USkeletalMeshComponent::SetEnablePerPolyCollision(bool bInEnablePerPolyCollision)
{
	if(bInEnablePerPolyCollision != bEnablePerPolyCollision)
	{
		DestroyPhysicsState();

		bEnablePerPolyCollision = bInEnablePerPolyCollision;

		if (IsRegistered())
		{
			CreatePhysicsState();
		}
	}
}

void USkeletalMeshComponent::SetEnableBodyGravity(bool bEnableGravity, FName BoneName)
{
	if (FBodyInstance* BI = GetBodyInstance(BoneName))
	{
		BI->SetEnableGravity(bEnableGravity);
	}
}

bool USkeletalMeshComponent::IsBodyGravityEnabled(FName BoneName)
{
	const FBodyInstance* BI = GetBodyInstance(BoneName);
	return BI && BI->bEnableGravity;
}

void USkeletalMeshComponent::SetEnableGravityOnAllBodiesBelow(bool bEnableGravity, FName BoneName, bool bIncludeSelf)
{
    ForEachBodyBelow(BoneName, bIncludeSelf, /*bSkipCustomPhysics=*/false, [bEnableGravity](FBodyInstance* BI)
	{
		BI->SetEnableGravity(bEnableGravity);
	});
}

//////////////////////////////////////////////////////////////////////////
// COLLISION

extern float DebugLineLifetime;

bool USkeletalMeshComponent::GetSquaredDistanceToCollision(const FVector& Point, float& OutSquaredDistance, FVector& OutClosestPointOnCollision) const
{
	OutClosestPointOnCollision = Point;
	bool bHasResult = false;

	for (int32 BodyIdx = 0; BodyIdx < Bodies.Num(); ++BodyIdx)
	{
		FBodyInstance* BodyInst = Bodies[BodyIdx];
		if (BodyInst && BodyInst->IsValidBodyInstance() && (BodyInst->GetCollisionEnabled() != ECollisionEnabled::NoCollision))
		{
			FVector ClosestPoint;
			float DistanceSqr = -1.f;

			if (!Bodies[BodyIdx]->GetSquaredDistanceToBody(Point, DistanceSqr, ClosestPoint))
			{
				// Invalid result, impossible to be better than ClosestPointDistance
				continue;
			}

			if (!bHasResult || (DistanceSqr < OutSquaredDistance))
			{
				bHasResult = true;
				OutSquaredDistance = DistanceSqr;
				OutClosestPointOnCollision = ClosestPoint;

				// If we're inside collision, we're not going to find anything better, so abort search we've got our best find.
				if (DistanceSqr <= UE_KINDA_SMALL_NUMBER)
				{
					break;
				}
			}
		}
	}

	return bHasResult;
}

DECLARE_CYCLE_STAT(TEXT("GetClosestPointOnPhysicsAsset"), STAT_GetClosestPointOnPhysicsAsset, STATGROUP_Physics);

bool USkeletalMeshComponent::GetClosestPointOnPhysicsAsset(const FVector& WorldPosition, FClosestPointOnPhysicsAsset& ClosestPointOnPhysicsAsset, bool bApproximate) const
{
	SCOPE_CYCLE_COUNTER(STAT_GetClosestPointOnPhysicsAsset);

	bool bSuccess = false;
	const UPhysicsAsset* PhysicsAsset = GetPhysicsAsset();
	const FReferenceSkeleton* RefSkeleton = GetSkeletalMeshAsset() ? &GetSkeletalMeshAsset()->GetRefSkeleton() : nullptr;
	if(PhysicsAsset && RefSkeleton)
	{
		const TArray<FTransform>& BoneTransforms = GetComponentSpaceTransforms();
		const bool bHasLeaderPoseComponent = LeaderPoseComponent.IsValid();
		const FVector ComponentPosition = GetComponentTransform().InverseTransformPosition(WorldPosition);
	
		float CurrentClosestDistance = FLT_MAX;
		int32 CurrentClosestBoneIndex = INDEX_NONE;
		const UBodySetup* CurrentClosestBodySetup = nullptr;

		for(const UBodySetup* BodySetupInstance : PhysicsAsset->SkeletalBodySetups)
		{
			ClosestPointOnPhysicsAsset.Distance = FLT_MAX;
			const FName BoneName = BodySetupInstance->BoneName;
			const int32 BoneIndex = RefSkeleton->FindBoneIndex(BoneName);
			if(BoneIndex != INDEX_NONE)
			{
				const FTransform BoneTM = bHasLeaderPoseComponent ? GetBoneTransform(BoneIndex) : BoneTransforms[BoneIndex];
				const float Dist = bApproximate ? (BoneTM.GetLocation() - ComponentPosition).SizeSquared() : BodySetupInstance->GetShortestDistanceToPoint(ComponentPosition, BoneTM);

				if (Dist < CurrentClosestDistance)
				{
					CurrentClosestDistance = Dist;
					CurrentClosestBoneIndex = BoneIndex;
					CurrentClosestBodySetup = BodySetupInstance;

					if(Dist <= 0.f) { break; }
				}
			}
		}

		if(CurrentClosestBoneIndex >= 0)
		{
			bSuccess = true;

			const FTransform BoneTM = bHasLeaderPoseComponent ? GetBoneTransform(CurrentClosestBoneIndex) : (BoneTransforms[CurrentClosestBoneIndex] * GetComponentTransform());
			ClosestPointOnPhysicsAsset.Distance = CurrentClosestBodySetup->GetClosestPointAndNormal(WorldPosition, BoneTM, ClosestPointOnPhysicsAsset.ClosestWorldPosition, ClosestPointOnPhysicsAsset.Normal);
			ClosestPointOnPhysicsAsset.BoneName = CurrentClosestBodySetup->BoneName;
		}
	}

	return bSuccess;
}

bool USkeletalMeshComponent::K2_GetClosestPointOnPhysicsAsset(const FVector& WorldPosition, FVector& ClosestWorldPosition, FVector& Normal, FName& BoneName, float& Distance) const
{
	FClosestPointOnPhysicsAsset ClosestPointOnPhysicsAsset;
	bool bSuccess = GetClosestPointOnPhysicsAsset(WorldPosition, ClosestPointOnPhysicsAsset, /*bApproximate =*/ false);
	if(bSuccess)
	{
		ClosestWorldPosition = ClosestPointOnPhysicsAsset.ClosestWorldPosition;
		Normal = ClosestPointOnPhysicsAsset.Normal;
		BoneName = ClosestPointOnPhysicsAsset.BoneName;
		Distance = ClosestPointOnPhysicsAsset.Distance;
	}
	else
	{
		ClosestWorldPosition = FVector::ZeroVector;
		Normal = FVector::ZeroVector;
		BoneName = NAME_None;
		Distance = -1;
	}

	return bSuccess;
}

bool USkeletalMeshComponent::LineTraceComponent(struct FHitResult& OutHit, const FVector Start, const FVector End, const struct FCollisionQueryParams& Params)
{
	UWorld* const World = GetWorld();
	bool bHaveHit = false;

	if (bEnablePerPolyCollision)
	{
		// Using PrimitiveComponent implementation
		//as it intersects against mesh polys.
		bHaveHit = UPrimitiveComponent::LineTraceComponent(OutHit, Start, End, Params);
	}
	else
	{
		float MinTime = UE_MAX_FLT;
		FHitResult Hit;
		for (int32 BodyIdx = 0; BodyIdx < Bodies.Num(); ++BodyIdx)
		{
			if (Bodies[BodyIdx] && Bodies[BodyIdx]->LineTrace(Hit, Start, End, Params.bTraceComplex, Params.bReturnPhysicalMaterial))
			{
				bHaveHit = true;
				if (MinTime > Hit.Time)
				{
					MinTime = Hit.Time;
					OutHit = Hit;
				}
			}
		}
	}

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	if(World && World->DebugDrawSceneQueries(Params.TraceTag))
	{
		TArray<FHitResult> Hits;
		if (bHaveHit)
		{
			Hits.Add(OutHit);
		}
		DrawLineTraces(GetWorld(), Start, End, Hits, DebugLineLifetime);
	}
#endif //!(UE_BUILD_SHIPPING || UE_BUILD_TEST)

	return bHaveHit;
}

bool USkeletalMeshComponent::SweepComponent( FHitResult& OutHit, const FVector Start, const FVector End, const FQuat& ShapeWorldRotation, const FCollisionShape& CollisionShape, bool bTraceComplex)
{
	bool bHaveHit = false;

	if (bEnablePerPolyCollision)
	{
		// Using PrimitiveComponent implementation
		//as it intersects against mesh polys.
		bHaveHit =  UPrimitiveComponent::SweepComponent(OutHit, Start, End, ShapeWorldRotation, CollisionShape, bTraceComplex);
	}
	else
	{
		FHitResult Hit;
		for (int32 BodyIdx = 0; BodyIdx < Bodies.Num(); ++BodyIdx)
		{
			if (Bodies[BodyIdx] && Bodies[BodyIdx]->Sweep(Hit, Start, End, ShapeWorldRotation, CollisionShape, bTraceComplex))
			{
				if (!bHaveHit || Hit.Time < OutHit.Time)
				{
					OutHit = Hit;
				}
				bHaveHit = true;
			}
		}
	}

	return bHaveHit;
}

bool USkeletalMeshComponent::ComponentOverlapComponentImpl(class UPrimitiveComponent* PrimComp,const FVector Pos,const FQuat& Quat,const struct FCollisionQueryParams& Params)
{
	//we do not support skeletal mesh vs skeletal mesh overlap test
	if (PrimComp->IsA<USkeletalMeshComponent>())
	{
		UE_LOGF(LogCollision, Warning, "ComponentOverlapComponent : (%ls) Does not support skeletalmesh with Physics Asset", *PrimComp->GetPathName());
		return false;
	}

	//We do not support skeletal mesh vs Instanced static meshes
	if (PrimComp->IsA<UInstancedStaticMeshComponent>())
	{
		UE_LOGF(LogCollision, Warning, "ComponentOverlapComponent : (%ls) Does not support skeletalmesh with Physics Asset", *PrimComp->GetPathName());
		return false;
	}

	if (FBodyInstance* BI = PrimComp->GetBodyInstance())
	{
		return BI->OverlapTestForBodies(Pos, Quat, Bodies);
	}

	return false;
}

bool USkeletalMeshComponent::OverlapComponent(const FVector& Pos, const FQuat& Rot, const FCollisionShape& CollisionShape) const
{
	for (FBodyInstance* Body : Bodies)
	{
		if (Body->OverlapTest(Pos, Rot, CollisionShape))
		{
			return true;
		}
	}

	return false;
}

bool USkeletalMeshComponent::ComponentOverlapMultiImpl(TArray<struct FOverlapResult>& OutOverlaps, const UWorld* World, const FVector& Pos, const FQuat& Quat, ECollisionChannel TestChannel, const struct FComponentQueryParams& Params, const struct FCollisionObjectQueryParams& ObjectQueryParams) const
{
	OutOverlaps.Reset();

	if (!Bodies.IsValidIndex(RootBodyData.BodyIndex))
	{
		return false;
	}

	const FTransform WorldToComponent(GetComponentTransform().Inverse());
	const FCollisionResponseParams ResponseParams(GetCollisionResponseToChannels());

	FComponentQueryParams ParamsWithSelf = Params;
	ParamsWithSelf.AddIgnoredComponent(this);

	bool bHaveBlockingHit = false;
	for (const FBodyInstance* Body : Bodies)
	{
		checkSlow(Body);
		if (Body->OverlapMulti(OutOverlaps, World, &WorldToComponent, Pos, Quat, TestChannel, ParamsWithSelf, ResponseParams, ObjectQueryParams))
		{
			bHaveBlockingHit = true;
		}
	}

	return bHaveBlockingHit;
}

void USkeletalMeshComponent::AddClothingBounds(FBoxSphereBounds& InOutBounds, const FTransform& LocalToWorld) const
{
	const int32 PredictedLOD = GetPredictedLODLevel();
	for (const FClothingSimulationInstance& ClothingSimulationInstance : ClothingSimulationInstances)
	{
		InOutBounds = InOutBounds + ClothingSimulationInstance.GetBounds(this).TransformBy(LocalToWorld);
	}
}

void USkeletalMeshComponent::RecreateClothingActors()
{
	CSV_SCOPED_TIMING_STAT(Animation, ClothInit);

	ReleaseAllClothingResources();

	const USkeletalMesh* const SkelMesh = GetSkeletalMeshAsset();
	if (SkelMesh == nullptr || !IsRegistered())
	{
		return;
	}

	// Only create cloth sim actors when the world is ready for it
	const UWorld* const World = GetWorld();
	const bool bIsWorldReadyForSimulation =
#if WITH_EDITORONLY_DATA
		World && (World->bShouldSimulatePhysics || bUpdateClothInEditor) && World->GetPhysicsScene();
#else
		World && World->bShouldSimulatePhysics && World->GetPhysicsScene();
#endif

	if (bIsWorldReadyForSimulation && bAllowClothActors && CVarEnableClothPhysics.GetValueOnGameThread())
	{
		const TArray<UClothingAssetBase*>& MeshClothingAssets = SkelMesh->GetMeshClothingAssets();
		if (MeshClothingAssets.Num())
		{
			UE_LOGF(LogSkeletalMesh, Log, "USkeletalMeshComponent: Recreating Clothing Actors for '%ls' with '%ls'", *GetName(), *SkelMesh->GetName());

			// Group assets per factory, as to only create one simulation per asset of the same type
			TArray<UClothingAssetBase*> ClothingAssetsInUse;
			SkelMesh->GetClothingAssetsInUse(ClothingAssetsInUse);

			TArray<FClothingSimulationInstance::FFactoryAssetGroup, TInlineAllocator<8>> FactoryAssetGroups;
			FactoryAssetGroups.Reserve(ClothingAssetsInUse.Num());

			for (int32 ClothingAssetIndex = 0; ClothingAssetIndex < MeshClothingAssets.Num(); ++ClothingAssetIndex)
			{
				const UClothingAssetBase* const ClothingAsset = MeshClothingAssets[ClothingAssetIndex];
				if (ClothingAsset && ClothingAsset->IsValid() && ClothingAssetsInUse.Contains(ClothingAsset))
				{
					if (UClothingSimulationFactory* const ClothingAssetFactory = UClothingSimulationFactory::GetClothingSimulationFactory(ClothingAsset))
					{
						FClothingSimulationInstance::FFactoryAssetGroup* FactoryAssetGroup = FactoryAssetGroups.FindByPredicate(
							[&ClothingAssetFactory](const FClothingSimulationInstance::FFactoryAssetGroup& FactoryAssetGroup)
							{
								return FactoryAssetGroup.GetClothingSimulationFactory() == ClothingAssetFactory;
							});
						if (!FactoryAssetGroup)
						{
							FactoryAssetGroup = &FactoryAssetGroups.Emplace_GetRef(ClothingAssetFactory, MeshClothingAssets.Num());
						}
						FactoryAssetGroup->AddClothingAsset(ClothingAsset, ClothingAssetIndex);
					}
				}
			}

			// Create clothing simulation instances
			checkf(!ClothingSimulationInstances.Num(), TEXT("ReleaseAllClothingResources() should have been called to clear all clothing simulation instances before this point."));
			ClothingSimulationInstances.Reserve(FactoryAssetGroups.Num());
			for (FClothingSimulationInstance::FFactoryAssetGroup& FactoryAssetGroup : FactoryAssetGroups)
			{
				// Initialize a new clothing simulation instance for this asset group, add actors, create interactors, ...etc
				FClothingSimulationInstance& ClothingSimulationInstance = ClothingSimulationInstances.Emplace_GetRef(this, FactoryAssetGroup);
			}

			if (ClothingSimulationInstances.Num())
			{
				// Retrieve the cloth sim data, or clear the data if the world isn't ready to sim
				WritebackClothingSimulationData();
			}
		}
	}
}

// UE_DEPRECATED(5.7, "Use ReleaseAllClothingResources instead.")
void USkeletalMeshComponent::RemoveAllClothingActors()
{
	if (ClothingSimulationInstances.Num())
	{
		// Can't destroy our actors if we're still simulating
		HandleExistingParallelClothSimulation();

		for (FClothingSimulationInstance& ClothingSimulationInstance : ClothingSimulationInstances)
		{
			ClothingSimulationInstance.RemoveAllClothingActors();
		}
	}
}

void USkeletalMeshComponent::ReleaseAllClothingResources()
{
	if (ClothingSimulationInstances.Num())
	{
		// Ensure no running simulation first
		HandleExistingParallelClothSimulation();

		for (FClothingSimulationInstance& ClothingSimulationInstance : ClothingSimulationInstances)
		{
#if WITH_CLOTH_COLLISION_DETECTION
			ClothingSimulationInstance.ClearExternalCollisions();
#endif // #if WITH_CLOTH_COLLISION_DETECTION
			
			ClothingSimulationInstance.RemoveAllClothingActors();
		}

		// Remove all clothing simulations
		ClothingSimulationInstances.Reset();
	}
}

void USkeletalMeshComponent::GetWindForCloth_GameThread(FVector& WindDirection, float& WindAdaption) const
{
	check(IsInGameThread());

	WindDirection = FVector::ZeroVector;
	WindAdaption = 2.f;	//not sure where this const comes from, but that's what the old code did
	
	UWorld* World = GetWorld();
	if(World && World->Scene)
	{
		// set wind
		if(IsWindEnabled())
		{
			FVector Position = GetComponentTransform().GetTranslation();

			float WindSpeed;
			float WindMinGust;
			float WindMaxGust;
			World->Scene->GetWindParameters_GameThread(Position, WindDirection, WindSpeed, WindMinGust, WindMaxGust);

			WindDirection *= WindSpeed;
			WindAdaption = FMath::Rand() % 20 * 0.1f; // make range from 0 to 2
		}
	}
}

#if WITH_CLOTH_COLLISION_DETECTION

// UE_DEPRECATED(5.7, "Use ClothCollisionSource::ExtractCollisions() instead.")
void USkeletalMeshComponent::ExtractCollisionsForCloth(
	USkeletalMeshComponent* SourceComponent, 
	UPhysicsAsset* PhysicsAsset, 
	USkeletalMeshComponent* DestClothComponent, 
	FClothCollisionData& OutCollisions, 
	FClothCollisionSource& ClothCollisionSource)
{
	if (ensure(ClothCollisionSource.SourceComponent.Get() == SourceComponent &&
		ClothCollisionSource.SourcePhysicsAsset.Get() == PhysicsAsset))
	{
		ClothCollisionSource.ExtractCollisions(DestClothComponent, OutCollisions);
	}
}


void USkeletalMeshComponent::CopyClothCollisionSources()
{
	FClothCollisionData ExternalCollisions;

	for (FClothCollisionSource& ClothCollisionSource : ClothCollisionSources)
	{
		ClothCollisionSource.ExtractCollisions(this, ExternalCollisions);
	}

	for (FClothingSimulationInstance& ClothingSimulationInstance : ClothingSimulationInstances)
	{
		ClothingSimulationInstance.AddExternalCollisions(ExternalCollisions);
	}
}

void USkeletalMeshComponent::ProcessClothCollisionWithEnvironment()
{
	// don't handle collision detection if this component is in editor
	if (!GetWorld()->IsGameWorld() || !ClothingSimulationInstances.Num())
	{
		return;
	}

	FClothCollisionData NewCollisionData;
	FEnvironmentalCollisions::AppendCollisionDataFromEnvironment(this, NewCollisionData);

	for (FClothingSimulationInstance& ClothingSimulationInstance : ClothingSimulationInstances)
	{
		ClothingSimulationInstance.AddExternalCollisions(NewCollisionData);
	}
}

#endif// #if WITH_CLOTH_COLLISION_DETECTION

void USkeletalMeshComponent::AddClothCollisionSource(USkeletalMeshComponent* InSourceComponent, UPhysicsAsset* InSourcePhysicsAsset)
{
#if WITH_CLOTH_COLLISION_DETECTION
	if(InSourceComponent && InSourcePhysicsAsset)
	{
		FClothCollisionSource* FoundCollisionSource = ClothCollisionSources.FindByPredicate(
			[InSourceComponent, InSourcePhysicsAsset](const FClothCollisionSource& InCollisionSource)
			{ 
				return InCollisionSource.SourceComponent.Get() == InSourceComponent && InCollisionSource.SourcePhysicsAsset.Get() == InSourcePhysicsAsset; 
			}
		);

		if(FoundCollisionSource == nullptr)
		{
			// Add an UpdateClothTransform delegate after the transform buffer flip, so that the cloths' transform gets updated when the component owning the cloth isn't moving, but the collision source is
			const FOnBoneTransformsFinalizedMultiCast::FDelegate OnBoneTransformsFinalizedDelegate =
				FOnBoneTransformsFinalizedMultiCast::FDelegate::CreateUObject(this, &USkeletalMeshComponent::UpdateClothTransform);

			// Add the new collision source
			ClothCollisionSources.Emplace(InSourceComponent, InSourcePhysicsAsset, OnBoneTransformsFinalizedDelegate);

			// Add prerequisite so we don't end up with a frame delay
			ClothTickFunction.AddPrerequisite(InSourceComponent, InSourceComponent->PrimaryComponentTick);
		}
	}
#endif
}

void USkeletalMeshComponent::RemoveClothCollisionSources(USkeletalMeshComponent* InSourceComponent)
{
#if WITH_CLOTH_COLLISION_DETECTION
	if (InSourceComponent)
	{
		const int32 NumRemoved = ClothCollisionSources.RemoveAll([InSourceComponent](const FClothCollisionSource& InCollisionSource)
			{
				return !InCollisionSource.SourceComponent.IsValid() || InCollisionSource.SourceComponent.Get() == InSourceComponent;
			});

		if (NumRemoved)
		{
			// Drop the tick prerequisite now that this source no longer contributes collisions
			ClothTickFunction.RemovePrerequisite(InSourceComponent, InSourceComponent->PrimaryComponentTick);
			UpdateClothTransform();
		}
	}
#endif
}

void USkeletalMeshComponent::RemoveClothCollisionSource(USkeletalMeshComponent* InSourceComponent, UPhysicsAsset* InSourcePhysicsAsset)
{
#if WITH_CLOTH_COLLISION_DETECTION
	if (InSourceComponent && InSourcePhysicsAsset)
	{
		const int32 NumRemoved = ClothCollisionSources.RemoveAll([InSourceComponent, InSourcePhysicsAsset](const FClothCollisionSource& InCollisionSource)
			{
				return !InCollisionSource.SourceComponent.IsValid() || (InCollisionSource.SourceComponent.Get() == InSourceComponent && InCollisionSource.SourcePhysicsAsset.Get() == InSourcePhysicsAsset);
			});

		if (NumRemoved)
		{
			// Only drop the tick prerequisite when no remaining entry still references this source component
			bool bRemovePrerequisite = true;
			for (const FClothCollisionSource& ClothCollisionSource : ClothCollisionSources)
			{
				if (ClothCollisionSource.SourceComponent.Get() == InSourceComponent)
				{
					bRemovePrerequisite = false;
					break;
				}
			}
			if (bRemovePrerequisite)
			{
				ClothTickFunction.RemovePrerequisite(InSourceComponent, InSourceComponent->PrimaryComponentTick);
			}
			UpdateClothTransform();
		}
	}
#endif
}

void USkeletalMeshComponent::ResetClothCollisionSources()
{
#if WITH_CLOTH_COLLISION_DETECTION
	if (ClothCollisionSources.Num())
	{
		// Drop tick prerequisites for any still-valid sources before clearing the array
		for (const FClothCollisionSource& ClothCollisionSource : ClothCollisionSources)
		{
			if (ClothCollisionSource.SourceComponent.IsValid())
			{
				ClothTickFunction.RemovePrerequisite(ClothCollisionSource.SourceComponent.Get(), ClothCollisionSource.SourceComponent.Get()->PrimaryComponentTick);
			}
		}
		ClothCollisionSources.Reset();
		UpdateClothTransform();
	}
#endif
}

void USkeletalMeshComponent::EndPhysicsTickComponent(FSkeletalMeshComponentEndPhysicsTickFunction& ThisTickFunction)
{
	//IMPORTANT!
	//
	// The decision on whether to use EndPhysicsTickComponent or not is made by ShouldRunEndPhysicsTick()
	// Any changes that are made to EndPhysicsTickComponent that affect whether it should be run or not
	// have to be reflected in ShouldRunEndPhysicsTick() as well
	
	// if physics is disabled on dedicated server, no reason to be here. 
	if (!bEnablePhysicsOnDedicatedServer && IsRunningDedicatedServer())
	{
		FinalizeBoneTransform();
		return;
	}

	if (IsRegistered() && IsSimulatingPhysics() && RigidBodyIsAwake())
	{
		if (bNotifySyncComponentToRBPhysics)
		{
			OnSyncComponentToRBPhysics();
		}

		SyncComponentToRBPhysics();
	}

	// this used to not run if not rendered, but that causes issues such as bounds not updated
	// causing it to not rendered, at the end, I think we should blend body positions
	// for example if you're only simulating, this has to happen all the time
	// whether looking at it or not, otherwise
	// @todo better solution is to check if it has moved by changing SyncComponentToRBPhysics to return true if anything modified
	// and run this if that is true or rendered
	// that will at least reduce the chance of mismatch
	// generally if you move your actor position, this has to happen to approximately match their bounds
	if (ShouldBlendPhysicsBones())
	{
		if (IsRegistered())
		{
			BlendInPhysicsInternal(ThisTickFunction);
		}
	}
}

void USkeletalMeshComponent::UpdateClothTransformImp()
{
#if WITH_CLOTH_COLLISION_DETECTION
	if (ClothingSimulationInstances.Num())
	{
		for (FClothingSimulationInstance& ClothingSimulationInstance : ClothingSimulationInstances)
		{
			// Component has moved, and there is something to simulate, update all external cloth collisions
			ClothingSimulationInstance.ClearExternalCollisions();
		}

		if (ClothCollisionSources.Num() > 0)
		{
			CopyClothCollisionSources();
		}

		if (bCollideWithEnvironment)
		{
			ProcessClothCollisionWithEnvironment();
		}
	}
#endif // WITH_CLOTH_COLLISION_DETECTION

#if !(UE_BUILD_SHIPPING)
	FTransform ComponentTransform = GetComponentTransform();
	if (ComponentTransform.GetRotation().ContainsNaN())
	{
		logOrEnsureNanError(TEXT("SkeletalMeshComponent::UpdateClothTransform found NaN in GetComponentTransform().GetRotation()"));
		ComponentTransform.SetRotation(FQuat(0.0f, 0.0f, 0.0f, 1.0f));
		SetComponentToWorld(ComponentTransform);
	}
	if (ComponentTransform.ContainsNaN())
	{
		logOrEnsureNanError(TEXT("SkeletalMeshComponent::UpdateClothTransform still found NaN in GetComponentTransform() (wasn't the rotation)"));
		SetComponentToWorld(FTransform::Identity);
	}
#endif
}

void USkeletalMeshComponent::UpdateClothTransform(ETeleportType TeleportType)
{
	//Note that it's not safe to run the update here. This is because cloth sim could still be running on another thread. We defer it
	bPendingClothTransformUpdate = true;
	PendingTeleportType = ((TeleportType > PendingTeleportType) ? TeleportType : PendingTeleportType); 
}

void USkeletalMeshComponent::UpdateRigidBodyScaling(ETeleportType TeleportType)
{
	if (bEnablePerPolyCollision)
	{
		return;
	}

	const FVector3f SceneRelativeScale = FVector3f(GetRelativeScale3D());

	// Physics does not support nonuniform scaling of the bodies.
	if (!SceneRelativeScale.IsUniform())
	{
		return;
	}

	// Scale didn't actually change - nothing to do.
	if ((PrevRootRigidBodyScale - SceneRelativeScale).IsNearlyZero())
	{
		return;
	}

	// The legacy code computes a per-frame change factor (SceneRelativeScale / PrevRootRigidBodyScale)
	// and applies it to every body. When the scale crosses zero that factor becomes 0 (going to zero) or
	// Inf/NaN (coming back from zero), and the NaN corrupts body scale/mass and joint transforms
	// irrecoverably. We therefore special-case the two zero crossings and only use the incremental factor
	// on the safe nonzero->nonzero path.
	const bool bGoingToZero = SceneRelativeScale.IsNearlyZero() && !PrevRootRigidBodyScale.IsNearlyZero();
	const bool bComingFromZero = PrevRootRigidBodyScale.IsNearlyZero() && !SceneRelativeScale.IsNearlyZero();

	if (bGoingToZero)
	{
		// NOTE: disabling physics here has two expected side effects for the scale-to-zero case: the
		// animation / post-process anim blueprint may take over and modify the body transforms while scaled
		// out, and a runtime SetAllBodiesSimulatePhysics(true) ragdoll returns in its physics-asset default state.
		SetAllBodiesSimulatePhysics(false);

	}
	else if (bComingFromZero)
	{
		// RecreatePhysicsState() re-instantiates the bodies from the current bone transforms, which already
		// carry the component's (target) world scale, so they come back at the correct scale. No post-creation
		// rescale is needed here.
		RecreatePhysicsState();
	}
	else
	{
		// Safe incremental path: PrevRootRigidBodyScale and SceneRelativeScale are both uniform and nonzero.
		const FVector RelativeScale3DLocal = FVector(SceneRelativeScale / PrevRootRigidBodyScale);

		// Defensive guard: must never feed a NaN/Inf factor into the bodies. That is the exact corruption
		// the codepaths above should handle. 
		if (RelativeScale3DLocal.ContainsNaN())
		{
			return;
		}

		for (FBodyInstance* BI : Bodies)
		{
			if (BI && BI->IsValidBodyInstance())
			{
				//TODO(Chaos): Using current scale to compute for desired scale may lead to drift when scales are continuously varied.
				// It may help if the scale can be pulled from somewhere else.
				if (BI->UpdateBodyScale(RelativeScale3DLocal * BI->Scale3D, true))
				{
					//Reset rotation as before.
					BI->SetBodyTransform(GetBoneTransform(BI->InstanceBoneIndex), TeleportType, true);
				}
			}
		}

		//Reset relative transform using the relative scale.
		for (int32 i = 0; i < Constraints.Num(); i++)
		{
			check(Constraints[i]);
			if (Chaos::FJointConstraint* JointConstraint = static_cast<Chaos::FJointConstraint*>(Constraints[i]->ConstraintHandle.Constraint))
			{
				Chaos::FTransformPair JointTransforms = JointConstraint->GetJointTransforms();
				JointTransforms[0].SetTranslation(JointTransforms[0].GetTranslation() * RelativeScale3DLocal);
				JointTransforms[1].SetTranslation(JointTransforms[1].GetTranslation() * RelativeScale3DLocal);

				JointConstraint->SetJointTransforms(JointTransforms);
			}
		}
	}

	PrevRootRigidBodyScale = SceneRelativeScale;
}

void USkeletalMeshComponent::CheckClothTeleport()
{
	// Get the root bone transform
	FMatrix CurRootBoneMat = GetBoneMatrix(0);

	ClothTeleportMode = UE::ClothingSimulation::TeleportHelpers::CalculateClothingTeleport(ClothTeleportMode, CurRootBoneMat, PrevRootBoneMatrix, bResetAfterTeleport, ClothTeleportDistThresholdSquared, ClothTeleportCosineThresholdInRad);

	PrevRootBoneMatrix = CurRootBoneMat;
}

FAutoConsoleTaskPriority CPrio_FParallelClothTask(
	TEXT("TaskGraph.TaskPriorities.ParallelClothTask"),
	TEXT("Task and thread priority for parallel cloth."),
	ENamedThreads::HighThreadPriority, // if we have high priority task threads, then use them...
	ENamedThreads::NormalTaskPriority, // .. at normal task priority
	ENamedThreads::HighTaskPriority // if we don't have hi pri threads, then use normal priority threads at high task priority instead
	);

class FParallelClothTask
{
	USkeletalMeshComponent& SkeletalMeshComponent;
	float DeltaTime;

public:
	FParallelClothTask(USkeletalMeshComponent& InSkeletalMeshComponent, float InDeltaTime)
		: SkeletalMeshComponent(InSkeletalMeshComponent)
		, DeltaTime(InDeltaTime)
	{
	}

	FORCEINLINE TStatId GetStatId() const
	{
		RETURN_QUICK_DECLARE_CYCLE_STAT(FParallelClothTask, STATGROUP_TaskGraphTasks);
	}
	static ENamedThreads::Type GetDesiredThread()
	{
		if (CVarEnableClothPhysicsUseTaskThread.GetValueOnAnyThread() != 0)
		{
			return CPrio_FParallelClothTask.Get();
		}
		return ENamedThreads::GameThread;
	}
	static ESubsequentsMode::Type GetSubsequentsMode()
	{
		return ESubsequentsMode::TrackSubsequents;
	}

	void DoTask(ENamedThreads::Type CurrentThread, const FGraphEventRef& MyCompletionGraphEvent)
	{
		FScopeCycleCounterUObject ContextScope(SkeletalMeshComponent.GetSkeletalMeshAsset());
		SCOPE_CYCLE_COUNTER(STAT_ClothTotalTime);
		CSV_SCOPED_TIMING_STAT(Animation, Cloth);

		for (FClothingSimulationInstance& ClothingSimulationInstance : SkeletalMeshComponent.ClothingSimulationInstances)
		{
			ClothingSimulationInstance.Simulate();
		}
	}
};

// This task runs after the clothing task to perform a writeback of data from the simulation to the
// component. This is done on the gamethread as that is where the data is to be valid (ensures no other
// component ticks will be accessing during the writeback)
class FParallelClothCompletionTask
{
	TWeakObjectPtr<USkeletalMeshComponent> SkeletalMeshComponent;

public:
	FParallelClothCompletionTask(TWeakObjectPtr<USkeletalMeshComponent> InSkeletalMeshComponent)
		: SkeletalMeshComponent(InSkeletalMeshComponent)
	{}

	FORCEINLINE TStatId GetStatId() const
	{
		RETURN_QUICK_DECLARE_CYCLE_STAT(FParallelClothCompletionTask, STATGROUP_TaskGraphTasks);
	}

	static ENamedThreads::Type GetDesiredThread()
	{
		return ENamedThreads::GameThread;
	}

	static ESubsequentsMode::Type GetSubsequentsMode()
	{
		return ESubsequentsMode::TrackSubsequents;
	}

	void DoTask(ENamedThreads::Type CurrentThread, const FGraphEventRef& MyCompletionGraphEvent)
	{
		SCOPE_CYCLE_COUNTER(STAT_ClothWriteback)
		// Perform the data writeback
		if(USkeletalMeshComponent* MeshComp = SkeletalMeshComponent.Get())
		{
			MeshComp->CompleteParallelClothSimulation();
		}
	}
};

bool USkeletalMeshComponent::RequiresPreEndOfFrameSync() const
{
	if (!ShouldUseClothEndTickWait()
		&& ClothingSimulationInstances.Num()
		&& CVarEnableClothPhysics.GetValueOnGameThread()
		&& !ShouldWaitForClothInTickFunction())
	{
		return true;
	}
	return Super::RequiresPreEndOfFrameSync();
}

void USkeletalMeshComponent::OnPreEndOfFrameSync()
{
	Super::OnPreEndOfFrameSync();

	if (!ShouldUseClothEndTickWait())
	{
		HandleExistingParallelClothSimulation();
	}
}

bool USkeletalMeshComponent::ShouldWaitForClothInTickFunction() const
{
	return bWaitForParallelClothTask || (CVarClothPhysicsTickWaitForParallelClothTask.GetValueOnAnyThread() != 0);
}

bool USkeletalMeshComponent::ShouldUseClothEndTickWait() const
{
	return CVarClothPhysicsUseEndTickWait.GetValueOnAnyThread() != 0;
}

const TMap<int32, FClothSimulData>& USkeletalMeshComponent::GetCurrentClothingData_GameThread() const
{
	// We require the cloth tick to wait for the simulation results if we want to use them for some reason other than rendering.
	if (!ShouldWaitForClothInTickFunction())
	{
		// Log a one-time warning
		UE_LOGF(LogSkeletalMesh, Warning, "Use of USkeletalMeshComponent::GetCurrentClothingData_GameThread requires that property bWaitForParallelClothTask be set to true");

		// Make it work for next frame
		const_cast<USkeletalMeshComponent*>(this)->bWaitForParallelClothTask = true;

		// Return an empty dataset
		return SEmptyClothSimulationData;
	}

	return ClothingSimulationInstances.Num() ? CurrentSimulationData : SEmptyClothSimulationData;
}

const TMap<int32, FClothSimulData>& USkeletalMeshComponent::GetCurrentClothingData_AnyThread() const
{
	// Called during EndOfFrameUpdates, usually in a parallel-for loop, where waiting on the task here would re-enter EOF state via the busy wait.
	// With p.ClothPhysics.UseEndTickWait=1 (default), FSkeletalMeshComponentClothEndTickFunction (TG_LastDemotable) waits before EOF runs, so the task is complete by the time this is called.
	// With p.ClothPhysics.UseEndTickWait=0 (kill switch), OnPreEndOfFrameSync runs the wait earlier in SendAllEndOfFrameUpdatesInternal.
	// Either way, this never blocks; if the task isn't complete yet we return the empty dataset.
	if (ClothingSimulationInstances.Num() && (!IsValidRef(ParallelClothTask) || ParallelClothTask->IsComplete()))
	{
		return CurrentSimulationData;
	}

	return SEmptyClothSimulationData;
}

void USkeletalMeshComponent::UpdateClothStateAndSimulate(float DeltaTime, FTickFunction& ThisTickFunction)
{
	check(IsInGameThread());

	// If disabled or no simulation
	if (CVarEnableClothPhysics.GetValueOnGameThread() == 0 || !ClothingSimulationInstances.Num() || bDisableClothSimulation)
	{
		return;
	}

	// If we simulate a clothing actor at 0s it will fill simulated positions and normals with NaNs.
	// we can skip all the work it is still doing, and get the desired result (frozen sim) by not
	// updating and simulating.
	if(DeltaTime == 0.0f)
	{
		return;
	}

	// Make sure we aren't already in flight from previous frame
	HandleExistingParallelClothSimulation();

	UpdateClothSimulationContext(DeltaTime);

	ParallelClothTask = TGraphTask<FParallelClothTask>::CreateTask(nullptr, ENamedThreads::GameThread).ConstructAndDispatchWhenReady(*this, DeltaTime);

	if (ShouldWaitForClothInTickFunction())
	{
		FGraphEventArray Prerequisites;
		Prerequisites.Add(ParallelClothTask);
		FGraphEventRef ClothCompletionEvent = TGraphTask<FParallelClothCompletionTask>::CreateTask(&Prerequisites, ENamedThreads::GameThread).ConstructAndDispatchWhenReady(this);
		ThisTickFunction.GetCompletionHandle()->DontCompleteUntil(ClothCompletionEvent);
	}
}

//This is the total cloth time split up among multiple computation (updating gpu, updating sim, etc...)
DECLARE_CYCLE_STAT(TEXT("Cloth Sim"), STAT_ClothSimTime, STATGROUP_Physics);

bool USkeletalMeshComponent::GetClothSimulatedPosition_GameThread(const FGuid& AssetGuid, int32 VertexIndex, FVector& OutSimulPos) const
{
	if(!GetSkeletalMeshAsset())
	{
		// Can't proceed without a mesh
		return false;
	}
		
	bool bSucceed = false;

	int32 AssetIndex = GetSkeletalMeshAsset()->GetClothingAssetIndex(AssetGuid);

	if(AssetIndex != INDEX_NONE)
	{
		const FClothSimulData* ActorData = GetCurrentClothingData_GameThread().Find(AssetIndex);

		if(ActorData && ActorData->Positions.IsValidIndex(VertexIndex))
		{
			OutSimulPos = (FVector)ActorData->Positions[VertexIndex];

			bSucceed = true;
		}
	}
	return bSucceed;
}

void USkeletalMeshComponent::TickClothing(float DeltaTime, FTickFunction& ThisTickFunction)
{
	bool bIsCompiling = false;
#if WITH_EDITOR
	bIsCompiling = (GetSkeletalMeshAsset() && GetSkeletalMeshAsset()->IsCompiling());
#endif
	if (GetSkeletalMeshAsset() == nullptr || !ClothingSimulationInstances.Num() || CVarEnableClothPhysics.GetValueOnGameThread() == 0 || bIsCompiling)
	{
		return;
	}

	// Use the component update flag to gate simulation to respect the always tick options
	static const bool bClothPhysicsTickWithMontages = (CVarClothPhysicsTickWithMontages.GetValueOnAnyThread() != 0);
	const bool bShouldTick = bClothPhysicsTickWithMontages ? 
		((VisibilityBasedAnimTickOption < EVisibilityBasedAnimTickOption::OnlyTickPoseWhenRendered) || bRecentlyRendered) :  // Legacy ticking behavior that ticked with montages
		(VisibilityBasedAnimTickOption == EVisibilityBasedAnimTickOption::AlwaysTickPoseAndRefreshBones ||
		VisibilityBasedAnimTickOption == EVisibilityBasedAnimTickOption::AlwaysTickPose ||
		bRecentlyRendered);

	if (bShouldTick)
	{
		UpdateClothStateAndSimulate(DeltaTime, ThisTickFunction);
	}
	else
	{
		ForceClothNextUpdateTeleportAndReset();
	}
}

void USkeletalMeshComponent::GetUpdateClothSimulationData_AnyThread(TMap<int32, FClothSimulData>& OutClothSimulData, FMatrix& OutLocalToWorld, float& OutClothBlendWeight) const
{
	OutLocalToWorld = GetComponentToWorld().ToMatrixWithScale();

	const USkeletalMeshComponent* const LeaderPoseSkeletalMeshComponent = Cast<USkeletalMeshComponent>(LeaderPoseComponent.Get());
	if (LeaderPoseSkeletalMeshComponent && bBindClothToLeaderComponent)
	{
		OutClothBlendWeight = ClothBlendWeight;
		OutClothSimulData = LeaderPoseSkeletalMeshComponent->GetCurrentClothingData_AnyThread();
	}
	else if (!bDisableClothSimulation && !bBindClothToLeaderComponent)
	{
		OutClothBlendWeight = ClothBlendWeight;
		OutClothSimulData = GetCurrentClothingData_AnyThread();
	}
	else
	{
		OutClothSimulData.Reset();
	}

	// Blend cloth out whenever the simulation data is invalid
	if (!OutClothSimulData.Num())
	{
		OutClothBlendWeight = 0.0f;
	}
}

void USkeletalMeshComponent::WaitForExistingParallelClothSimulation_GameThread()
{
	// Should only kick new parallel cloth simulations from game thread, so should be safe to also wait for existing ones there.
	check(IsInGameThread());
	HandleExistingParallelClothSimulation();
}

void USkeletalMeshComponent::DebugDrawClothing(FPrimitiveDrawInterface* PDI)
{
#if WITH_EDITOR && ENABLE_DRAW_DEBUG

	if (ClothingSimulationInstances.Num())
	{
		FClothingSystemEditorInterfaceModule& ClothingEditorModule = FModuleManager::LoadModuleChecked<FClothingSystemEditorInterfaceModule>(TEXT("ClothingSystemEditorInterface"));

		for (const FClothingSimulationInstance& ClothingSimulationInstance : ClothingSimulationInstances)
		{
			const FName ClothingSimulationFactoryName = ClothingSimulationInstance.GetClothingSimulationFactory()->GetClass()->GetFName();
			if (ISimulationEditorExtender* const Extender = ClothingEditorModule.GetSimulationEditorExtender(ClothingSimulationFactoryName))
			{
				Extender->DebugDrawSimulation(ClothingSimulationInstance.GetClothingSimulation(), this, PDI);
			}
		}
	}

#endif
}

void USkeletalMeshComponent::DebugDrawClothingTexts(FCanvas* Canvas, const FSceneView* SceneView)
{
#if WITH_EDITOR && ENABLE_DRAW_DEBUG

	if (ClothingSimulationInstances.Num())
	{
		FClothingSystemEditorInterfaceModule& ClothingEditorModule = FModuleManager::LoadModuleChecked<FClothingSystemEditorInterfaceModule>(TEXT("ClothingSystemEditorInterface"));

		for (const FClothingSimulationInstance& ClothingSimulationInstance : ClothingSimulationInstances)
		{
			const FName ClothingSimulationFactoryName = ClothingSimulationInstance.GetClothingSimulationFactory()->GetClass()->GetFName();
			if (ISimulationEditorExtender* const Extender = ClothingEditorModule.GetSimulationEditorExtender(ClothingSimulationFactoryName))
			{
				Extender->DebugDrawSimulationTexts(ClothingSimulationInstance.GetClothingSimulation(), this, Canvas, SceneView);
			}
		}
	}

#endif
}

void USkeletalMeshComponent::SetAllMassScale(float InMassScale)
{
	// Apply mass scale to each child body
	for(FBodyInstance* BI : Bodies)
	{
		if (BI->IsValidBodyInstance())
		{
			BI->SetMassScale(InMassScale);
		}
	}
}


float USkeletalMeshComponent::GetMass() const
{
	float Mass = 0.0f;
	for (int32 i=0; i < Bodies.Num(); ++i)
	{
		FBodyInstance* BI = Bodies[i];

		if (BI->IsValidBodyInstance())
		{
			Mass += BI->GetBodyMass();
		}
	}
	return Mass;
}

float USkeletalMeshComponent::GetBoneMass(FName BoneName, bool bScaleMass) const
{
	if (FBodyInstance* BI = GetBodyInstance(BoneName))
	{
		float Scale = 1.0f;
		if (bScaleMass)
		{
			Scale = BI->MassScale;
		}
		return Scale*BI->GetBodyMass();
	}

	return 0.0f;
}

FVector USkeletalMeshComponent::GetSkeletalCenterOfMass() const
{
	FVector Location = FVector::ZeroVector;
	float Mass = 0.0f;
	for (int32 i = 0; i < Bodies.Num(); i++)
	{
		FBodyInstance* BI = Bodies[i];
		if (BI->IsValidBodyInstance())
		{
			float BodyMass = BI->MassScale*BI->GetBodyMass();
			Location += BodyMass*BI->GetCOMPosition();
			Mass += BodyMass;
		}
	}

	return Location / Mass;
}

void USkeletalMeshComponent::SetAllUseCCD(bool InUseCCD)
{
	// Apply CCD setting to each child body
	for (FBodyInstance* BI : Bodies)
	{
		if (BI->IsValidBodyInstance())
		{
			BI->SetUseCCD(InUseCCD);
		}
	}
}

// blueprint callable methods 
float USkeletalMeshComponent::GetClothMaxDistanceScale() const
{
	return ClothMaxDistanceScale;
}

void USkeletalMeshComponent::SetClothMaxDistanceScale(float Scale)
{
	ClothMaxDistanceScale = Scale;
}

void USkeletalMeshComponent::ResetClothTeleportMode()
{
	ClothTeleportMode = EClothingTeleportMode::None;
}

void USkeletalMeshComponent::ForceClothNextUpdateTeleport()
{
	ClothTeleportMode = EClothingTeleportMode::Teleport;
}

void USkeletalMeshComponent::ForceClothNextUpdateTeleportAndReset()
{
	ClothTeleportMode = EClothingTeleportMode::TeleportAndReset;
}

FTransform USkeletalMeshComponent::GetComponentTransformFromBodyInstance(FBodyInstance* UseBI)
{
	if (PhysicsTransformUpdateMode == EPhysicsTransformUpdateMode::SimulationUpatesComponentTransform)
	{
		// undo root transform so that it only moves according to what actor itself suppose to move
		const FTransform& BodyTransform = UseBI->GetUnrealWorldTransform();
		return RootBodyData.TransformToRoot * BodyTransform;
	}
	else
	{
		return GetComponentTransform();
	}
}


Chaos::FPhysicsObject* USkeletalMeshComponent::GetPhysicsObjectById(Chaos::FPhysicsObjectId Id) const
{
	if (!Bodies.IsValidIndex(Id) || !Bodies[Id] || !Bodies[Id]->GetPhysicsActor())
	{
		return nullptr;
	}
	return Bodies[Id]->GetPhysicsActor()->GetPhysicsObject();
}

Chaos::FPhysicsObject* USkeletalMeshComponent::GetPhysicsObjectByName(const FName& Name) const
{
	FBodyInstance* Body = GetBodyInstance(Name);
	if (!Body || !Body->GetPhysicsActor())
	{
		return nullptr;
	}

	return Body->GetPhysicsActor()->GetPhysicsObject();
}

TArray<Chaos::FPhysicsObject*> USkeletalMeshComponent::GetAllPhysicsObjects() const
{
	TArray<Chaos::FPhysicsObject*> Objects;
	Objects.Reserve(Bodies.Num());
	for (int32 Index = 0; Index < Bodies.Num(); ++Index)
	{
		Objects.Add(GetPhysicsObjectById(Index));
	}
	return Objects;
}

#undef LOCTEXT_NAMESPACE
