// Copyright Epic Games, Inc. All Rights Reserved.
#include "SkeletalMeshDeformerHelpers.h"

#include "DataDrivenShaderPlatformInfo.h"
#include "PrimitiveSceneInfo.h"
#include "RenderGraphBuilder.h"
#include "RenderGraphResources.h"
#include "RenderGraphUtils.h"
#include "RenderingThread.h"
#include "SceneInterface.h"
#include "SkeletalMeshSceneProxy.h"
#include "SkeletalRenderGPUSkin.h"

FRHIShaderResourceView* FSkeletalMeshDeformerHelpers::GetBoneBufferForReading(
	FSkeletalMeshObject const* InMeshObject,
	int32 InLodIndex,
	int32 InSectionIndex,
	bool bInPreviousFrame)
{
	if (InMeshObject->IsCPUSkinned())
	{
		return nullptr;
	}

	FSkeletalMeshObjectGPUSkin const* MeshObjectGPU = static_cast<FSkeletalMeshObjectGPUSkin const*>(InMeshObject);
	FGPUBaseSkinVertexFactory const* BaseVertexFactory = MeshObjectGPU->GetBaseSkinVertexFactory(InLodIndex, InSectionIndex);
	bool bHasBoneBuffer = BaseVertexFactory != nullptr && BaseVertexFactory->GetShaderData().HasBoneBufferForReading(bInPreviousFrame);
	if (!bHasBoneBuffer)
	{
		return nullptr;
	}

	return BaseVertexFactory->GetShaderData().GetBoneBufferForReading(bInPreviousFrame).VertexBufferSRV;
}

FRHIShaderResourceView* FSkeletalMeshDeformerHelpers::GetMorphTargetBufferForReading(
	const FSkeletalMeshObject* InMeshObject,
	int32 InLodIndex,
	bool bInPreviousFrame)
{
	if (InMeshObject->IsCPUSkinned())
	{
		return nullptr;
	}

	const FSkeletalMeshObjectGPUSkin* MeshObjectGPU = static_cast<const FSkeletalMeshObjectGPUSkin *>(InMeshObject);

	const FMorphVertexBuffer* MorphVertexBuffer = MeshObjectGPU->LODs[InLodIndex].MorphVertexBufferPool->TryGetMorphVertexBufferForReading(bInPreviousFrame);
	if (!MorphVertexBuffer)
	{
		return nullptr;
	}
	return MorphVertexBuffer->GetSRV();
}

FSkeletalMeshDeformerHelpers::FClothBuffers FSkeletalMeshDeformerHelpers::GetClothBuffersForReading(
	FSkeletalMeshObject const* InMeshObject,
	int32 InLodIndex,
	int32 InSectionIndex,
	uint32 InFrameNumber,
	bool bInPreviousFrame)
{
	if (InMeshObject->IsCPUSkinned())
	{
		return FClothBuffers();
	}

	FSkeletalMeshObjectGPUSkin const* MeshObjectGPU = static_cast<FSkeletalMeshObjectGPUSkin const*>(InMeshObject);
	FGPUBaseSkinVertexFactory const* BaseVertexFactory = MeshObjectGPU->GetBaseSkinVertexFactory(InLodIndex, InSectionIndex);
	FGPUBaseSkinAPEXClothVertexFactory const* ClothVertexFactory = BaseVertexFactory != nullptr ? BaseVertexFactory->GetClothVertexFactory() : nullptr;

	if (ClothVertexFactory == nullptr || !ClothVertexFactory->GetClothShaderData().HasClothBufferForReading(bInPreviousFrame))
	{
		return FClothBuffers();
	}

	FSkeletalMeshRenderData const& SkeletalMeshRenderData = InMeshObject->GetSkeletalMeshRenderData();
	FSkeletalMeshLODRenderData const* LodRenderData = SkeletalMeshRenderData.GetPendingFirstLOD(InLodIndex);
	FSkelMeshRenderSection const& RenderSection = LodRenderData->RenderSections[InSectionIndex];

	FClothBuffers Ret;
	Ret.ClothInfluenceBuffer = ClothVertexFactory->GetClothBuffer();
	Ret.ClothInfluenceBufferOffset = ClothVertexFactory->GetClothIndexOffset(RenderSection.BaseVertexIndex);
	Ret.ClothSimulatedPositionAndNormalBuffer = ClothVertexFactory->GetClothShaderData().GetClothBufferForReading(bInPreviousFrame).VertexBufferSRV;
	Ret.ClothToLocal = ClothVertexFactory->GetClothShaderData().GetClothToLocalForReading(bInPreviousFrame);
	return Ret;
}

FRDGBuffer* FSkeletalMeshDeformerHelpers::GetAllocatedPositionBuffer(FRDGBuilder& GraphBuilder, FSkeletalMeshObject* InMeshObject, int32 InLodIndex)
{
	if (InMeshObject->IsCPUSkinned())
	{
		return nullptr;
	}

	FSkeletalMeshObjectGPUSkin* MeshObjectGPU = static_cast<FSkeletalMeshObjectGPUSkin*>(InMeshObject);

	FMeshDeformerGeometry& DeformerGeometry = MeshObjectGPU->GetDeformerGeometry(InLodIndex);
	
	FRDGBuffer* PositionBuffer = nullptr;
	if (DeformerGeometry.Position)
	{
		PositionBuffer = GraphBuilder.RegisterExternalBuffer(DeformerGeometry.Position);
	}
	return PositionBuffer;
}

FRDGBuffer* FSkeletalMeshDeformerHelpers::GetAllocatedTangentBuffer(FRDGBuilder& GraphBuilder, FSkeletalMeshObject* InMeshObject, int32 InLodIndex)
{
	if (InMeshObject->IsCPUSkinned())
	{
		return nullptr;
	}

	FSkeletalMeshObjectGPUSkin* MeshObjectGPU = static_cast<FSkeletalMeshObjectGPUSkin*>(InMeshObject);

	FMeshDeformerGeometry& DeformerGeometry = MeshObjectGPU->GetDeformerGeometry(InLodIndex);
	
	FRDGBuffer* TangentBuffer = nullptr;
	if (DeformerGeometry.Tangent)
	{
		TangentBuffer = GraphBuilder.RegisterExternalBuffer(DeformerGeometry.Tangent);
	}
	return TangentBuffer;
}

FRDGBuffer* FSkeletalMeshDeformerHelpers::GetAllocatedColorBuffer(FRDGBuilder& GraphBuilder, FSkeletalMeshObject* InMeshObject, int32 InLodIndex)
{
	if (InMeshObject->IsCPUSkinned())
	{
		return nullptr;
	}

	FSkeletalMeshObjectGPUSkin* MeshObjectGPU = static_cast<FSkeletalMeshObjectGPUSkin*>(InMeshObject);

	FMeshDeformerGeometry& DeformerGeometry = MeshObjectGPU->GetDeformerGeometry(InLodIndex);
	
	FRDGBuffer* ColorBuffer = nullptr;
	if (DeformerGeometry.Color)
	{
		ColorBuffer = GraphBuilder.RegisterExternalBuffer(DeformerGeometry.Color);
	}
	return ColorBuffer;
}

int32 FSkeletalMeshDeformerHelpers::GetIndexOfFirstAvailableSection(FSkeletalMeshObject* InMeshObject, int32 InLodIndex)
{
	int32 Result = INDEX_NONE;
	const TArray<FSkelMeshRenderSection>& RenderSections = InMeshObject->GetRenderSections(InLodIndex);
	for (int32 Index = 0; Index < RenderSections.Num(); Index++)
	{
		if (!RenderSections[Index].bDisabled)
		{
			Result = Index;
			break;
		}
	}

	return Result;
}

FSkeletalMeshObject* FSkeletalMeshDeformerHelpers::GetSkeletalMeshObject(FSceneInterface* InSceneInterface, FPrimitiveComponentId InComponentId)
{
	if (!InSceneInterface || !InComponentId.IsValid())
	{
		return nullptr;
	}

	FPrimitiveSceneInfo* PrimitiveSceneInfo = InSceneInterface->GetPrimitiveSceneInfo(InComponentId);
	if (!PrimitiveSceneInfo || !PrimitiveSceneInfo->Proxy || !PrimitiveSceneInfo->Proxy->IsSkinnedMesh())
	{
		return nullptr;
	}

	FSkeletalMeshObject* MeshObject = static_cast<FSkeletalMeshSceneProxy*>(PrimitiveSceneInfo->Proxy)->GetMeshObject();
	if (!MeshObject || MeshObject->IsCPUSkinned())
	{
		return nullptr;
	}

	return MeshObject;
}

FRDGBuffer* FSkeletalMeshDeformerHelpers::AllocateVertexFactoryPositionBuffer(FRDGBuilder& GraphBuilder, FRDGExternalAccessQueue& ExternalAccesQueue, FSkeletalMeshObject* InMeshObject, int32 InLodIndex, TCHAR const* InBufferName)
{
	if (InMeshObject->IsCPUSkinned())
	{
		return nullptr;
	}

	FSkeletalMeshRenderData const& SkeletalMeshRenderData = InMeshObject->GetSkeletalMeshRenderData();
	FSkeletalMeshLODRenderData const* LodRenderData = SkeletalMeshRenderData.GetPendingFirstLOD(InLodIndex);
	const int32 NumVertices = LodRenderData->GetNumVertices();
	const FRDGBufferDesc BufferDesc = FRDGBufferDesc::CreateBufferDesc(PosBufferBytesPerElement, NumVertices * PosBufferElementMultiplier);

	FSkeletalMeshObjectGPUSkin* MeshObjectGPU = static_cast<FSkeletalMeshObjectGPUSkin*>(InMeshObject);
	FGPUBaseSkinVertexFactory const* BaseVertexFactory = MeshObjectGPU->GetBaseSkinVertexFactory(InLodIndex, GetIndexOfFirstAvailableSection(InMeshObject, InLodIndex));
	uint32 Frame = BaseVertexFactory->GetShaderData().UpdatedFrameNumber;

	FMeshDeformerGeometry& DeformerGeometry = MeshObjectGPU->GetDeformerGeometry(InLodIndex);

	FRDGBuffer* PositionBuffer = nullptr;
	if (DeformerGeometry.Position.IsValid() && Frame == DeformerGeometry.PositionUpdatedFrame)
	{
		PositionBuffer = GraphBuilder.RegisterExternalBuffer(DeformerGeometry.Position);
		GraphBuilder.UseInternalAccessMode(PositionBuffer);
	}
	else
	{
		const bool bIsMatchingDescription = DeformerGeometry.Position.IsValid() && DeformerGeometry.Position->Desc == BufferDesc;
		const bool bIsMatchingPreviousDescription = DeformerGeometry.PrevPosition.IsValid() && DeformerGeometry.PrevPosition->Desc == BufferDesc;

		if (bIsMatchingDescription && bIsMatchingPreviousDescription)
		{
			// Flip position buffers and return the current one.
			DeformerGeometry.PrevPosition.Swap(DeformerGeometry.Position);
			DeformerGeometry.PrevPositionSRV.Swap(DeformerGeometry.PositionSRV);
			
			PositionBuffer = GraphBuilder.RegisterExternalBuffer(DeformerGeometry.Position);
			GraphBuilder.UseInternalAccessMode(PositionBuffer);
		}
		else
		{
			if (bIsMatchingDescription)
			{
				DeformerGeometry.PrevPosition = MoveTemp(DeformerGeometry.Position);
				DeformerGeometry.PrevPositionSRV = MoveTemp(DeformerGeometry.PositionSRV);
			}
			else
			{
				DeformerGeometry.PrevPosition = nullptr;
				DeformerGeometry.PrevPositionSRV = nullptr;
			}

			PositionBuffer = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateBufferDesc(PosBufferBytesPerElement, NumVertices * PosBufferElementMultiplier), InBufferName, ERDGBufferFlags::None);
			DeformerGeometry.Position = GraphBuilder.ConvertToExternalBuffer(PositionBuffer);

			DeformerGeometry.PositionSRV = DeformerGeometry.Position->GetOrCreateSRV(GraphBuilder.RHICmdList, FRHIBufferSRVCreateInfo(PF_R32_FLOAT));
		}

#if RHI_RAYTRACING
		// Update ray tracing geometry whenever we recreate the position buffer.
		FSkeletalMeshRenderData& SkelMeshRenderData = MeshObjectGPU->GetSkeletalMeshRenderData();
		FSkeletalMeshLODRenderData& LODModel = SkelMeshRenderData.LODRenderData[InLodIndex];

		const int32 NumSections = InMeshObject->GetRenderSections(InLodIndex).Num();

		TArray<FBufferRHIRef> VertexBuffers;
		VertexBuffers.SetNum(NumSections);
		for (int32 SectionIndex = 0; SectionIndex < NumSections; ++SectionIndex)
		{
			VertexBuffers[SectionIndex] = DeformerGeometry.Position->GetRHI();
		}

		MeshObjectGPU->UpdateRayTracingGeometry(GraphBuilder.RHICmdList, LODModel, InLodIndex, VertexBuffers);
#endif // RHI_RAYTRACING
	}

	DeformerGeometry.PositionUpdatedFrame = Frame;

	ExternalAccesQueue.AddUnique(PositionBuffer, ERHIAccess::VertexOrIndexBuffer | ERHIAccess::SRVMask);

	return PositionBuffer;
}

FRDGBuffer* FSkeletalMeshDeformerHelpers::AllocateVertexFactoryPositionBuffer(FRDGBuilder& GraphBuilder, FSkeletalMeshObject* InMeshObject, int32 InLodIndex, bool bInLodJustChanged, TCHAR const* InBufferName)
{
	FRDGExternalAccessQueue Queue;
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FRDGBuffer* Buffer = AllocateVertexFactoryPositionBuffer(GraphBuilder, Queue, InMeshObject, InLodIndex, InBufferName);
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	Queue.Submit(GraphBuilder);
	return Buffer;
}

FRDGBuffer* FSkeletalMeshDeformerHelpers::AllocateVertexFactoryTangentBuffer(FRDGBuilder& GraphBuilder, FRDGExternalAccessQueue& ExternalAccesQueue, FSkeletalMeshObject* InMeshObject, int32 InLodIndex, TCHAR const* InBufferName)
{
	if (InMeshObject->IsCPUSkinned())
	{
		return nullptr;
	}

	FSkeletalMeshRenderData const& SkeletalMeshRenderData = InMeshObject->GetSkeletalMeshRenderData();
	FSkeletalMeshLODRenderData const* LodRenderData = SkeletalMeshRenderData.GetPendingFirstLOD(InLodIndex);
	const int32 NumVertices = LodRenderData->GetNumVertices();
	const FRDGBufferDesc BufferDesc = FRDGBufferDesc::CreateBufferDesc(TangentBufferBytesPerElement, NumVertices * TangentBufferElementMultiplier);

	FSkeletalMeshObjectGPUSkin* MeshObjectGPU = static_cast<FSkeletalMeshObjectGPUSkin*>(InMeshObject);
	FGPUBaseSkinVertexFactory const* BaseVertexFactory = MeshObjectGPU->GetBaseSkinVertexFactory(InLodIndex, GetIndexOfFirstAvailableSection(InMeshObject, InLodIndex));
	uint32 Frame = BaseVertexFactory->GetShaderData().UpdatedFrameNumber;

	FMeshDeformerGeometry& DeformerGeometry = MeshObjectGPU->GetDeformerGeometry(InLodIndex);

	FRDGBuffer* TangentBuffer = nullptr;
	if (DeformerGeometry.Tangent.IsValid() && DeformerGeometry.Tangent->Desc == BufferDesc)
	{
		TangentBuffer = GraphBuilder.RegisterExternalBuffer(DeformerGeometry.Tangent);
		GraphBuilder.UseInternalAccessMode(TangentBuffer);
	}
	else
	{
		TangentBuffer = GraphBuilder.CreateBuffer(BufferDesc, InBufferName, ERDGBufferFlags::None);
		DeformerGeometry.Tangent = GraphBuilder.ConvertToExternalBuffer(TangentBuffer);
		const EPixelFormat TangentsFormat = IsOpenGLPlatform(GMaxRHIShaderPlatform) ? PF_R16G16B16A16_SINT : PF_R16G16B16A16_SNORM;
		DeformerGeometry.TangentSRV = DeformerGeometry.Tangent->GetOrCreateSRV(GraphBuilder.RHICmdList, FRHIBufferSRVCreateInfo(TangentsFormat));
	}

	DeformerGeometry.TangentUpdatedFrame = Frame;

	ExternalAccesQueue.AddUnique(TangentBuffer, ERHIAccess::VertexOrIndexBuffer | ERHIAccess::SRVMask);

	return TangentBuffer;
}

FRDGBuffer* FSkeletalMeshDeformerHelpers::AllocateVertexFactoryTangentBuffer(FRDGBuilder& GraphBuilder, FSkeletalMeshObject* InMeshObject, int32 InLodIndex, TCHAR const* InBufferName)
{
	FRDGExternalAccessQueue Queue;
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FRDGBuffer* Buffer = AllocateVertexFactoryTangentBuffer(GraphBuilder, Queue, InMeshObject, InLodIndex, InBufferName);
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	Queue.Submit(GraphBuilder);
	return Buffer;
}

FRDGBuffer* FSkeletalMeshDeformerHelpers::AllocateVertexFactoryColorBuffer(FRDGBuilder& GraphBuilder, FRDGExternalAccessQueue& ExternalAccesQueue, FSkeletalMeshObject* InMeshObject, int32 InLodIndex, TCHAR const* InBufferName)
{
	if (InMeshObject->IsCPUSkinned())
	{
		return nullptr;
	}

	FSkeletalMeshRenderData const& SkeletalMeshRenderData = InMeshObject->GetSkeletalMeshRenderData();
	FSkeletalMeshLODRenderData const* LodRenderData = SkeletalMeshRenderData.GetPendingFirstLOD(InLodIndex);
	const int32 NumVertices = LodRenderData->GetNumVertices();
	const FRDGBufferDesc BufferDesc = FRDGBufferDesc::CreateBufferDesc(ColorBufferBytesPerElement, NumVertices);

	FSkeletalMeshObjectGPUSkin* MeshObjectGPU = static_cast<FSkeletalMeshObjectGPUSkin*>(InMeshObject);
	FGPUBaseSkinVertexFactory const* BaseVertexFactory = MeshObjectGPU->GetBaseSkinVertexFactory(InLodIndex, GetIndexOfFirstAvailableSection(InMeshObject, InLodIndex));
	uint32 Frame = BaseVertexFactory->GetShaderData().UpdatedFrameNumber;

	FMeshDeformerGeometry& DeformerGeometry = MeshObjectGPU->GetDeformerGeometry(InLodIndex);

	FRDGBuffer* ColorBuffer = nullptr;
	if (DeformerGeometry.Color.IsValid() && DeformerGeometry.Color->Desc == BufferDesc)
	{
		ColorBuffer = GraphBuilder.RegisterExternalBuffer(DeformerGeometry.Color);
		GraphBuilder.UseInternalAccessMode(ColorBuffer);
	}
	else
	{
		ColorBuffer = GraphBuilder.CreateBuffer(BufferDesc, InBufferName, ERDGBufferFlags::None);
		DeformerGeometry.Color = GraphBuilder.ConvertToExternalBuffer(ColorBuffer);
		DeformerGeometry.ColorSRV = DeformerGeometry.Color->GetOrCreateSRV(GraphBuilder.RHICmdList, FRHIBufferSRVCreateInfo(PF_R8G8B8A8));
	}

	DeformerGeometry.ColorUpdatedFrame = Frame;

	ExternalAccesQueue.AddUnique(ColorBuffer, ERHIAccess::VertexOrIndexBuffer | ERHIAccess::SRVMask);

	return ColorBuffer;
}

FRDGBuffer* FSkeletalMeshDeformerHelpers::AllocateVertexFactoryColorBuffer(FRDGBuilder& GraphBuilder, FSkeletalMeshObject* InMeshObject, int32 InLodIndex, TCHAR const* InBufferName)
{
	FRDGExternalAccessQueue Queue;
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FRDGBuffer* Buffer = AllocateVertexFactoryColorBuffer(GraphBuilder, Queue, InMeshObject, InLodIndex, InBufferName);
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	Queue.Submit(GraphBuilder);
	return Buffer;
}

void FSkeletalMeshDeformerHelpers::UpdateVertexFactoryBufferOverrides(FRHICommandListBase& RHICmdList, FSkeletalMeshObject* InMeshObject, int32 InLodIndex, bool bInvalidatePreviousPosition)
{
	if (InMeshObject->IsCPUSkinned())
	{
		return;
	}

	FSkeletalMeshObjectGPUSkin* MeshObjectGPU = static_cast<FSkeletalMeshObjectGPUSkin*>(InMeshObject);
	FMeshDeformerGeometry& DeformerGeometry = MeshObjectGPU->GetDeformerGeometry(InLodIndex);

	FGPUSkinPassthroughVertexFactory::FAddVertexAttributeDesc Desc;
	Desc.FrameNumber = DeformerGeometry.PositionUpdatedFrame;

	bool bAssignedAttributes = false;

	if (DeformerGeometry.PositionSRV)
	{
		Desc.StreamBuffers[FGPUSkinPassthroughVertexFactory::VertexPosition] = DeformerGeometry.Position->GetRHI();
		Desc.SRVs[FGPUSkinPassthroughVertexFactory::Position] = DeformerGeometry.PositionSRV;
		Desc.SRVs[FGPUSkinPassthroughVertexFactory::PreviousPosition] = bInvalidatePreviousPosition ? nullptr : DeformerGeometry.PrevPositionSRV.GetReference();
		bAssignedAttributes = true;
	}
	if (DeformerGeometry.TangentSRV)
	{
		Desc.StreamBuffers[FGPUSkinPassthroughVertexFactory::VertexTangent] = DeformerGeometry.Tangent->GetRHI();
		Desc.SRVs[FGPUSkinPassthroughVertexFactory::Tangent] = DeformerGeometry.TangentSRV;
		bAssignedAttributes = true;
	}
	if (DeformerGeometry.ColorSRV)
	{
		Desc.StreamBuffers[FGPUSkinPassthroughVertexFactory::VertexColor] = DeformerGeometry.Color->GetRHI();
		Desc.SRVs[FGPUSkinPassthroughVertexFactory::Color] = DeformerGeometry.ColorSRV;
		bAssignedAttributes = true;
	}

	if (!bAssignedAttributes)
	{
		return;
	}

	const FSkeletalMeshObjectGPUSkin::FSkeletalMeshObjectLOD& LOD = MeshObjectGPU->LODs[InLodIndex];
	FGPUBaseSkinVertexFactory const* BaseVertexFactory = MeshObjectGPU->GetBaseSkinVertexFactory(InLodIndex, GetIndexOfFirstAvailableSection(InMeshObject, InLodIndex));
	FGPUSkinPassthroughVertexFactory* TargetVertexFactory = LOD.GPUSkinVertexFactories.PassthroughVertexFactory.Get();

	// The passthrough vertex factory should exist if we got this far, but prefer skipping the update to crashing if that assumption fails.
	if (ensure(TargetVertexFactory))
	{
		TargetVertexFactory->SetVertexAttributes(RHICmdList, BaseVertexFactory, Desc);
	}
}

void FSkeletalMeshDeformerHelpers::UpdateVertexFactoryBufferOverrides(FRDGBuilder& GraphBuilder, FSkeletalMeshObject* InMeshObject, int32 InLodIndex, bool bInvalidatePreviousPosition)
{
	GraphBuilder.AddPass({}, ERDGPassFlags::None, [InMeshObject, InLodIndex, bInvalidatePreviousPosition](FRHICommandList& RHICmdList)
	{
		UpdateVertexFactoryBufferOverrides(RHICmdList, InMeshObject, InLodIndex, bInvalidatePreviousPosition);
	});
}

void FSkeletalMeshDeformerHelpers::UpdateVertexFactoryBufferOverrides(FRHICommandListBase& RHICmdList, FSkeletalMeshObject* InMeshObject, int32 InLodIndex)
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	UpdateVertexFactoryBufferOverrides(RHICmdList, InMeshObject, InLodIndex, false);
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

void FSkeletalMeshDeformerHelpers::UpdateVertexFactoryBufferOverrides(FSkeletalMeshObject* InMeshObject, int32 InLodIndex)
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	UpdateVertexFactoryBufferOverrides(FRHICommandListImmediate::Get(), InMeshObject, InLodIndex);
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

FRHIShaderResourceView* FSkeletalMeshDeformerHelpers::GetMorphTargetBufferForReading(FSkeletalMeshObject const* InMeshObject, int32 InLodIndex,
	int32 InSectionIndex, uint32 InFrameNumber, bool bInPreviousFrame)
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	return GetMorphTargetBufferForReading(InMeshObject, InLodIndex, bInPreviousFrame);
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

void FSkeletalMeshDeformerHelpers::ResetVertexFactoryBufferOverrides(FSkeletalMeshObject* InMeshObject, int32 LODIndex)
{
	if (InMeshObject->IsCPUSkinned())
	{
		return;
	}

	FRHICommandListBase& RHICmdList = FRHICommandListImmediate::Get();

	FSkeletalMeshObjectGPUSkin* MeshObjectGPU = static_cast<FSkeletalMeshObjectGPUSkin*>(InMeshObject);
	FMeshDeformerGeometry& DeformerGeometry = MeshObjectGPU->GetDeformerGeometry(LODIndex);

	// This can be called per frame even when already reset. So early out if we don't need to do anything.
	const bool bIsReset = DeformerGeometry.PositionUpdatedFrame == 0 && DeformerGeometry.TangentUpdatedFrame == 0 && DeformerGeometry.ColorUpdatedFrame == 0;
	if (bIsReset)
	{
		return;
	}

	// Reset stored buffers.
	DeformerGeometry.Reset();

	// Reset vertex factories.
	const FSkeletalMeshObjectGPUSkin::FSkeletalMeshObjectLOD& LOD = MeshObjectGPU->LODs[LODIndex];
	FGPUSkinPassthroughVertexFactory* TargetVertexFactory = LOD.GPUSkinVertexFactories.PassthroughVertexFactory.Get();

	// The passthrough vertex factory should exist if we got this far, but prefer skipping the update to crashing if that assumption fails.
	if (ensure(TargetVertexFactory))
	{
		TargetVertexFactory->ResetVertexAttributes(RHICmdList);
	}

#if RHI_RAYTRACING
	// Reset ray tracing geometry.
	FSkeletalMeshRenderData& SkelMeshRenderData = MeshObjectGPU->GetSkeletalMeshRenderData();
	FSkeletalMeshLODRenderData& LODModel = SkelMeshRenderData.LODRenderData[LODIndex];
	FBufferRHIRef VertexBuffer = LODModel.StaticVertexBuffers.PositionVertexBuffer.VertexBufferRHI;

	TArray<FBufferRHIRef> VertexBuffers;
	const int32 NumSections = InMeshObject->GetRenderSections(LODIndex).Num();
	VertexBuffers.Init(VertexBuffer, NumSections);
	MeshObjectGPU->UpdateRayTracingGeometry(RHICmdList, LODModel, LODIndex, VertexBuffers);
#endif // RHI_RAYTRACING
}
