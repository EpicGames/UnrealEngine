// Copyright Epic Games, Inc. All Rights Reserved.
#include "Engine/SkeletalMeshEditorData.h"


#include UE_INLINE_GENERATED_CPP_BY_NAME(SkeletalMeshEditorData)

#if WITH_EDITORONLY_DATA

#include "Rendering/SkeletalMeshLODImporterData.h"

#endif //WITH_EDITORONLY_DATA

#define LOCTEXT_NAMESPACE "SkeltalMeshEditorData"

DEFINE_LOG_CATEGORY_STATIC(LogSkeletalMeshEditorData, Log, All);

USkeletalMeshEditorData::USkeletalMeshEditorData(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{}

#if WITH_EDITORONLY_DATA

void USkeletalMeshEditorData::PostLoad()
{
	ClearFlags(RF_Standalone);
	Super::PostLoad();
}

FRawSkeletalMeshBulkData& USkeletalMeshEditorData::GetLODImportedData(int32 LODIndex)
{

	//Create missing default item
	check(LODIndex >= 0);
	if (LODIndex >= RawSkeletalMeshBulkDatas.Num())
	{
		const int32 AddItemCount = 1 + (LODIndex - RawSkeletalMeshBulkDatas.Num());
		RawSkeletalMeshBulkDatas.Reserve(AddItemCount);
		while (LODIndex >= RawSkeletalMeshBulkDatas.Num())
		{
			RawSkeletalMeshBulkDatas.Add(MakeShared<FRawSkeletalMeshBulkData>());
		}
	}

	check(RawSkeletalMeshBulkDatas.IsValidIndex(LODIndex));
	//Return the Data
	return RawSkeletalMeshBulkDatas[LODIndex].Get();
}

bool USkeletalMeshEditorData::IsLODImportDataValid(int32 LODIndex)
{
	return RawSkeletalMeshBulkDatas.IsValidIndex(LODIndex);
}

bool USkeletalMeshEditorData::RemoveLODImportedData(int32 LODIndex)
{
	if(RawSkeletalMeshBulkDatas.IsValidIndex(LODIndex))
	{
		RawSkeletalMeshBulkDatas.RemoveAt(LODIndex);
		return true;
	}
	return false;
}

void USkeletalMeshEditorData::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);
	
	if (!Ar.IsLoading() && !Ar.IsSaving() && Ar.IsObjectReferenceCollector())
	{
		//There is no UObject reference in the skeletal mesh editor data
		return;
	}

	//Serialize all LODs Raw imported source data
	FRawSkeletalMeshBulkData::Serialize(Ar, RawSkeletalMeshBulkDatas, this);
}

#endif //WITH_EDITORONLY_DATA

#undef LOCTEXT_NAMESPACE

