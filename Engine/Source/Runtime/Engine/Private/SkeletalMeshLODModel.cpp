// Copyright Epic Games, Inc. All Rights Reserved.

#include "Rendering/SkeletalMeshLODModel.h"

#if WITH_EDITOR
#include "EngineLogs.h"
#include "EngineUtils.h"
#include "Engine/SkeletalMesh.h"
#include "Engine/SkinnedAssetCommon.h"
#include "Rendering/MultiSizeIndexContainer.h"
#include "Rendering/SkeletalMeshVertexBuffer.h"
#include "Rendering/ColorVertexBuffer.h"
#include "Rendering/SkinWeightVertexBuffer.h"
#include "SkeletalMeshLegacyCustomVersions.h"
#include "UObject/RenderingObjectVersion.h"
#include "Serialization/MemoryWriter.h"
#include "UObject/FortniteMainBranchObjectVersion.h"
#include "SkeletalMeshAttributes.h"
#include "ReferenceSkeleton.h"


/*-----------------------------------------------------------------------------
FSoftSkinVertex
-----------------------------------------------------------------------------*/

/**
* Serializer
*
* @param Ar - archive to serialize with
* @param V - vertex to serialize
* @return archive that was used
*/
FArchive& operator<<(FArchive& Ar, FSoftSkinVertex& V)
{
	Ar << V.Position;

	if (Ar.CustomVer(FRenderingObjectVersion::GUID) < FRenderingObjectVersion::IncreaseNormalPrecision)
	{
		FDeprecatedSerializedPackedNormal Temp;
		Ar << Temp;
		V.TangentX = Temp;
		Ar << Temp;
		V.TangentY = Temp;
		Ar << Temp;
		V.TangentZ = Temp;
	}
	else
	{
		Ar << V.TangentX << V.TangentY << V.TangentZ;
	}

	for (int32 UVIdx = 0; UVIdx < MAX_TEXCOORDS; ++UVIdx)
	{
		Ar << V.UVs[UVIdx];
	}

	Ar << V.Color;

	if (Ar.IsLoading())
	{
		FMemory::Memzero(V.InfluenceBones);
		FMemory::Memzero(V.InfluenceWeights);
	}

	// serialize bone and weight uint8 arrays in order
	// this is required when serializing as bulk data memory (see TArray::BulkSerialize notes)
	const bool bBeforeIncreaseBoneIndexLimitPerChunk = Ar.CustomVer(FAnimObjectVersion::GUID) < FAnimObjectVersion::IncreaseBoneIndexLimitPerChunk;
	for (uint32 InfluenceIndex = 0; InfluenceIndex < MAX_INFLUENCES_PER_STREAM; InfluenceIndex++)
	{
		if (Ar.IsLoading() && bBeforeIncreaseBoneIndexLimitPerChunk)
		{
			uint8 BoneIndex = 0;
			Ar << BoneIndex;
			V.InfluenceBones[InfluenceIndex] = BoneIndex;
		}
		else
		{
			Ar << V.InfluenceBones[InfluenceIndex];
		}
	}

	if (Ar.UEVer() >= VER_UE4_SUPPORT_8_BONE_INFLUENCES_SKELETAL_MESHES)
	{
		for (uint32 InfluenceIndex = MAX_INFLUENCES_PER_STREAM; InfluenceIndex < EXTRA_BONE_INFLUENCES; InfluenceIndex++)
		{
			if (Ar.IsLoading() && bBeforeIncreaseBoneIndexLimitPerChunk)
			{
				uint8 BoneIndex = 0;
				Ar << BoneIndex;
				V.InfluenceBones[InfluenceIndex] = BoneIndex;
			}
			else
			{
				Ar << V.InfluenceBones[InfluenceIndex];
			}
		}
	}

	if (Ar.CustomVer(FAnimObjectVersion::GUID) >= FAnimObjectVersion::UnlimitedBoneInfluences)
	{
		for (uint32 InfluenceIndex = EXTRA_BONE_INFLUENCES; InfluenceIndex < MAX_TOTAL_INFLUENCES; InfluenceIndex++)
		{
			Ar << V.InfluenceBones[InfluenceIndex];
		}
	}

	if (!Ar.IsLoading() || Ar.CustomVer(FUE5MainStreamObjectVersion::GUID) >= FUE5MainStreamObjectVersion::IncreasedSkinWeightPrecision)
	{
		for (uint32 InfluenceIndex = 0; InfluenceIndex < MAX_TOTAL_INFLUENCES; InfluenceIndex++)
		{
			Ar << V.InfluenceWeights[InfluenceIndex];
		}
	}
	else
	{
		uint32 MaxInfluences = MAX_INFLUENCES_PER_STREAM;
		if (Ar.CustomVer(FAnimObjectVersion::GUID) >= FAnimObjectVersion::UnlimitedBoneInfluences)
		{
			MaxInfluences = MAX_TOTAL_INFLUENCES;
		}
		else if (Ar.UEVer() >= VER_UE4_SUPPORT_8_BONE_INFLUENCES_SKELETAL_MESHES)
		{
			MaxInfluences = EXTRA_BONE_INFLUENCES;
		}
		
		uint8 OldInfluence = 0;
		for (uint32 InfluenceIndex = 0; InfluenceIndex < MaxInfluences; InfluenceIndex++)
		{
			Ar << OldInfluence;
			V.InfluenceWeights[InfluenceIndex] = (static_cast<uint16>(OldInfluence) << 8) | OldInfluence;
		}
	}

	return Ar;
}

static_assert(sizeof(uint16) == sizeof(FBoneIndexType));

bool FSoftSkinVertex::GetRigidWeightBone(uint16& OutBoneIndex) const
{
	bool bIsRigid = false;

	for (int32 WeightIdx = 0; WeightIdx < MAX_TOTAL_INFLUENCES; WeightIdx++)
	{
		if (InfluenceWeights[WeightIdx] == std::numeric_limits<uint16>::max())
		{
			bIsRigid = true;
			OutBoneIndex = InfluenceBones[WeightIdx];
			break;
		}
	}

	return bIsRigid;
}

uint16 FSoftSkinVertex::GetMaximumWeight() const
{
	uint16 MaxInfluenceWeight = 0;

	for (int32 Index = 0; Index < MAX_TOTAL_INFLUENCES; Index++)
	{
		const uint16 Weight = InfluenceWeights[Index];

		if (Weight > MaxInfluenceWeight)
		{
			MaxInfluenceWeight = Weight;
		}
	}

	return MaxInfluenceWeight;
}

/** Legacy 'rigid' skin vertex */
struct FLegacyRigidSkinVertex
{
	FVector3f		Position;
	FVector3f		TangentX;	// Tangent, U-direction
	FVector3f		TangentY;	// Binormal, V-direction
	FVector3f		TangentZ;	// Normal
	FVector2f		UVs[MAX_TEXCOORDS]; // UVs
	FColor			Color;		// Vertex color.
	uint8			Bone;

	friend FArchive& operator<<(FArchive& Ar, FLegacyRigidSkinVertex& V)
	{
		Ar << V.Position;

		if (Ar.CustomVer(FRenderingObjectVersion::GUID) < FRenderingObjectVersion::IncreaseNormalPrecision)
		{
			FDeprecatedSerializedPackedNormal Temp;
			Ar << Temp;
			V.TangentX = Temp;
			Ar << Temp;
			V.TangentY = Temp;
			Ar << Temp;
			V.TangentZ = Temp;
		}
		else
		{
			Ar << V.TangentX << V.TangentY << V.TangentZ;
		}

		for (int32 UVIdx = 0; UVIdx < MAX_TEXCOORDS; ++UVIdx)
		{
			Ar << V.UVs[UVIdx];
		}

		Ar << V.Color;
		Ar << V.Bone;

		return Ar;
	}

	/** Util to convert from legacy */
	void ConvertToSoftVert(FSoftSkinVertex& DestVertex)
	{
		DestVertex.Position = Position;
		DestVertex.TangentX = TangentX;
		DestVertex.TangentY = TangentY;
		DestVertex.TangentZ = TangentZ;
		// store the sign of the determinant in TangentZ.W
		DestVertex.TangentZ.W = GetBasisDeterminantSign((FVector)TangentX, (FVector)TangentY, (FVector)TangentZ);

		// copy all texture coordinate sets
		for(int32 i = 0; i < MAX_TEXCOORDS; ++i)
		{
			DestVertex.UVs[i] = FVector2f(UVs[i]);
		}

		DestVertex.Color = Color;
		DestVertex.InfluenceBones[0] = Bone;
		DestVertex.InfluenceWeights[0] = std::numeric_limits<uint16>::max();
		for (int32 InfluenceIndex = 1; InfluenceIndex < MAX_TOTAL_INFLUENCES; InfluenceIndex++)
		{
			DestVertex.InfluenceBones[InfluenceIndex] = 0;
			DestVertex.InfluenceWeights[InfluenceIndex] = 0;
		}
	}
};


/**
* Calculate max # of bone influences used by this skel mesh chunk
*/
void FSkelMeshSection::CalcMaxBoneInfluences()
{
	// if we only have rigid verts then there is only one bone
	MaxBoneInfluences = 1;
	// iterate over all the soft vertices for this chunk and find max # of bones used
	for (int32 VertIdx = 0; VertIdx < SoftVertices.Num(); VertIdx++)
	{
		FSoftSkinVertex& SoftVert = SoftVertices[VertIdx];

		// calc # of bones used by this soft skinned vertex
		int32 BonesUsed = 0;
		for (int32 InfluenceIdx = 0; InfluenceIdx < MAX_TOTAL_INFLUENCES; InfluenceIdx++)
		{
			if (SoftVert.InfluenceWeights[InfluenceIdx] > 0)
			{
				BonesUsed++;
			}
		}
		// reorder bones so that there aren't any unused influence entries within the [0,BonesUsed] range
		for (int32 InfluenceIdx = 0; InfluenceIdx < BonesUsed; InfluenceIdx++)
		{
			if (SoftVert.InfluenceWeights[InfluenceIdx] == 0)
			{
				for (int32 ExchangeIdx = InfluenceIdx + 1; ExchangeIdx < MAX_TOTAL_INFLUENCES; ExchangeIdx++)
				{
					if (SoftVert.InfluenceWeights[ExchangeIdx] != 0)
					{
						Exchange(SoftVert.InfluenceWeights[InfluenceIdx], SoftVert.InfluenceWeights[ExchangeIdx]);
						Exchange(SoftVert.InfluenceBones[InfluenceIdx], SoftVert.InfluenceBones[ExchangeIdx]);
						break;
					}
				}
			}
		}

		// maintain max bones used
		MaxBoneInfluences = FMath::Max(MaxBoneInfluences, BonesUsed);
	}
}

/**
* Calculate if this skel mesh section needs 16-bit bone indices
*/
void FSkelMeshSection::CalcUse16BitBoneIndex()
{
	bUse16BitBoneIndex = false;
	FBoneIndexType MaxBoneIndex = 0;
	for (int32 VertIdx = 0; VertIdx < SoftVertices.Num(); VertIdx++)
	{
		FSoftSkinVertex& SoftVert = SoftVertices[VertIdx];
		for (int32 InfluenceIdx = 0; InfluenceIdx < MAX_TOTAL_INFLUENCES; InfluenceIdx++)
		{
			MaxBoneIndex = FMath::Max(SoftVert.InfluenceBones[InfluenceIdx], MaxBoneIndex);
			if (MaxBoneIndex > 255)
			{
				bUse16BitBoneIndex = true;
				return;
			}
		}
	}
}

// Serialization.
FArchive& operator<<(FArchive& Ar, FSkelMeshSection& S)
{
	Ar.UsingCustomVersion(FEditorObjectVersion::GUID);
	Ar.UsingCustomVersion(FReleaseObjectVersion::GUID);
	Ar.UsingCustomVersion(FRenderingObjectVersion::GUID);
	Ar.UsingCustomVersion(FAnimObjectVersion::GUID); // Also used by FSoftSkinVertex serializer
	Ar.UsingCustomVersion(FSkeletalMeshCustomVersion::GUID);
	Ar.UsingCustomVersion(FUE5MainStreamObjectVersion::GUID);
	Ar.UsingCustomVersion(FRecomputeTangentCustomVersion::GUID);
	Ar.UsingCustomVersion(FOverlappingVerticesCustomVersion::GUID);
	Ar.UsingCustomVersion(FUE5ReleaseStreamObjectVersion::GUID);

	// When data is cooked for server platform some of the
	// variables are not serialized so that they're always
	// set to their initial values (for safety)
	FStripDataFlags StripFlags(Ar);

	Ar << S.MaterialIndex;

	if (Ar.CustomVer(FSkeletalMeshCustomVersion::GUID) < FSkeletalMeshCustomVersion::CombineSectionWithChunk)
	{
		uint16 DummyChunkIndex;
		Ar << DummyChunkIndex;
	}

	if (!StripFlags.IsAudioVisualDataStripped())
	{
		Ar << S.BaseIndex;
		Ar << S.NumTriangles;
	}

	if (Ar.CustomVer(FSkeletalMeshCustomVersion::GUID) < FSkeletalMeshCustomVersion::RemoveTriangleSorting)
	{
		uint8 DummyTriangleSorting;
		Ar << DummyTriangleSorting;
	}

	// for clothing info
	if (Ar.UEVer() >= VER_UE4_APEX_CLOTH)
	{
		// Load old 'disabled' flag on sections, as this was used to identify legacy clothing sections for conversion
		if (Ar.CustomVer(FSkeletalMeshCustomVersion::GUID) < FSkeletalMeshCustomVersion::DeprecateSectionDisabledFlag)
		{
			Ar << S.bLegacyClothingSection_DEPRECATED;
		}

		// No longer serialize this if it's not used to map sections any more.
		if(Ar.CustomVer(FSkeletalMeshCustomVersion::GUID) < FSkeletalMeshCustomVersion::RemoveDuplicatedClothingSections)
		{
			Ar << S.CorrespondClothSectionIndex_DEPRECATED;
		}
	}

	if (Ar.UEVer() >= VER_UE4_APEX_CLOTH_LOD)
	{
		if (Ar.CustomVer(FSkeletalMeshCustomVersion::GUID) < FSkeletalMeshCustomVersion::RemoveEnableClothLOD)
		{
			uint8 DummyEnableClothLOD;
			Ar << DummyEnableClothLOD;
		}
	}

	if (Ar.CustomVer(FRecomputeTangentCustomVersion::GUID) >= FRecomputeTangentCustomVersion::RuntimeRecomputeTangent)
	{
		Ar << S.bRecomputeTangent;
	}

	if (Ar.CustomVer(FRecomputeTangentCustomVersion::GUID) >= FRecomputeTangentCustomVersion::RecomputeTangentVertexColorMask)
	{
		Ar << S.RecomputeTangentsVertexMaskChannel;
	}
	else
	{
		// Our default is not to use vertex color as mask
		S.RecomputeTangentsVertexMaskChannel = ESkinVertexColorChannel::None;
	}

	if (Ar.CustomVer(FEditorObjectVersion::GUID) >= FEditorObjectVersion::RefactorMeshEditorMaterials)
	{
		Ar << S.bCastShadow;
	}
	else
	{
		S.bCastShadow = true;
	}

	if (Ar.CustomVer(FUE5MainStreamObjectVersion::GUID) >= FUE5MainStreamObjectVersion::SkelMeshSectionVisibleInRayTracingFlagAdded)
	{
		Ar << S.bVisibleInRayTracing;
	}
	else
	{
		// default is to be visible in ray tracing - which is consistent with behaviour before adding this member
		S.bVisibleInRayTracing = true;
	}

	if (Ar.CustomVer(FSkeletalMeshCustomVersion::GUID) >= FSkeletalMeshCustomVersion::CombineSectionWithChunk)
	{

		if (!StripFlags.IsAudioVisualDataStripped())
		{
			// This is so that BaseVertexIndex is never set to anything else that 0 (for safety)
			Ar << S.BaseVertexIndex;
		}

		if (!StripFlags.IsEditorDataStripped() && !(Ar.IsFilterEditorOnly() && Ar.IsCountingMemory()) && !Ar.IsObjectReferenceCollector())
		{
			// For backwards compat, read rigid vert array into array
			TArray<FLegacyRigidSkinVertex> LegacyRigidVertices;
			if (Ar.IsLoading() && Ar.CustomVer(FSkeletalMeshCustomVersion::GUID) < FSkeletalMeshCustomVersion::CombineSoftAndRigidVerts)
			{
				Ar << LegacyRigidVertices;
			}

			Ar << S.SoftVertices;

			// Once we have read in SoftVertices, convert and insert legacy rigid verts (if present) at start
			const int32 NumRigidVerts = LegacyRigidVertices.Num();
			if (NumRigidVerts > 0 && Ar.CustomVer(FSkeletalMeshCustomVersion::GUID) < FSkeletalMeshCustomVersion::CombineSoftAndRigidVerts)
			{
				S.SoftVertices.InsertUninitialized(0, NumRigidVerts);

				for (int32 VertIdx = 0; VertIdx < NumRigidVerts; VertIdx++)
				{
					LegacyRigidVertices[VertIdx].ConvertToSoftVert(S.SoftVertices[VertIdx]);
				}
			}
		}

		// If loading content newer than CombineSectionWithChunk but older than SaveNumVertices, update NumVertices here
		if (Ar.IsLoading() && Ar.CustomVer(FSkeletalMeshCustomVersion::GUID) < FSkeletalMeshCustomVersion::SaveNumVertices)
		{
			if (!StripFlags.IsAudioVisualDataStripped())
			{
				S.NumVertices = S.SoftVertices.Num();
			}
			else
			{
				UE_LOGF(LogSkeletalMesh, Warning, "Cannot set FSkelMeshSection::NumVertices for older content, loading in non-editor build.");
				S.NumVertices = 0;
			}
		}

		if (Ar.IsLoading() && Ar.CustomVer(FAnimObjectVersion::GUID) < FAnimObjectVersion::IncreaseBoneIndexLimitPerChunk)
		{
			// Previous versions only supported 8-bit bone indices and bUse16BitBoneIndex wasn't serialized 
			S.CalcUse16BitBoneIndex();
			check(!S.bUse16BitBoneIndex);
		}
		else
		{
			Ar << S.bUse16BitBoneIndex;
		}

		Ar << S.BoneMap;

		if (Ar.CustomVer(FSkeletalMeshCustomVersion::GUID) >= FSkeletalMeshCustomVersion::SaveNumVertices)
		{
			Ar << S.NumVertices;
		}

		// Removed NumRigidVertices and NumSoftVertices
		if (Ar.CustomVer(FSkeletalMeshCustomVersion::GUID) < FSkeletalMeshCustomVersion::CombineSoftAndRigidVerts)
		{
			int32 DummyNumRigidVerts, DummyNumSoftVerts;
			Ar << DummyNumRigidVerts;
			Ar << DummyNumSoftVerts;

			if (DummyNumRigidVerts + DummyNumSoftVerts != S.SoftVertices.Num())
			{
				UE_LOGF(LogSkeletalMesh, Error, "Legacy NumSoftVerts + NumRigidVerts != SoftVertices.Num()");
			}
		}

		Ar << S.MaxBoneInfluences;

#if WITH_EDITOR
		// If loading content where we need to recalc 'max bone influences' instead of using loaded version, do that now
		if (!StripFlags.IsEditorDataStripped() && Ar.IsLoading() && Ar.CustomVer(FSkeletalMeshCustomVersion::GUID) < FSkeletalMeshCustomVersion::RecalcMaxBoneInfluences)
		{
			S.CalcMaxBoneInfluences();
		}
#endif

		if (Ar.CustomVer(FUE5ReleaseStreamObjectVersion::GUID) < FUE5ReleaseStreamObjectVersion::AddClothMappingLODBias)
		{
			constexpr int32 ClothLODBias = 0;  // There isn't any cloth LOD bias prior to this version
			S.ClothMappingDataLODs.SetNum(1);
			Ar << S.ClothMappingDataLODs[ClothLODBias];
		}
		else
		{
			Ar << S.ClothMappingDataLODs;
		}

		// We no longer need the positions and normals for a clothing sim mesh to be stored in sections, so throw that data out
		if(Ar.CustomVer(FSkeletalMeshCustomVersion::GUID) < FSkeletalMeshCustomVersion::RemoveDuplicatedClothingSections)
		{
			TArray<FVector> DummyArray;
			Ar << DummyArray;
			Ar << DummyArray;
		}

		Ar << S.CorrespondClothAssetIndex;

		if (Ar.CustomVer(FSkeletalMeshCustomVersion::GUID) < FSkeletalMeshCustomVersion::NewClothingSystemAdded)
		{
			int16 DummyClothAssetSubmeshIndex;
			Ar << DummyClothAssetSubmeshIndex;
		}
		else
		{
			Ar << S.ClothingData;
		}

		if (Ar.CustomVer(FOverlappingVerticesCustomVersion::GUID) >= FOverlappingVerticesCustomVersion::DetectOVerlappingVertices)
		{
			Ar << S.OverlappingVertices;
		}

		if(Ar.CustomVer(FReleaseObjectVersion::GUID) >= FReleaseObjectVersion::AddSkeletalMeshSectionDisable)
		{
			Ar << S.bDisabled;
		}

		if (Ar.CustomVer(FSkeletalMeshCustomVersion::GUID) >= FSkeletalMeshCustomVersion::SectionIgnoreByReduceAdded)
		{
			Ar << S.GenerateUpToLodIndex;
		}
		else if(Ar.IsLoading())
		{
			S.GenerateUpToLodIndex = -1;
		}

		if (Ar.CustomVer(FEditorObjectVersion::GUID) >= FEditorObjectVersion::SkeletalMeshBuildRefactor)
		{
			Ar << S.OriginalDataSectionIndex;
			Ar << S.ChunkedParentSectionIndex;
		}
		else if (Ar.IsLoading())
		{
			S.OriginalDataSectionIndex = INDEX_NONE;
			S.ChunkedParentSectionIndex = INDEX_NONE;
		}
		return Ar;
	}

	return Ar;
}

void FSkelMeshSection::DeclareCustomVersions(FArchive& Ar)
{
	Ar.UsingCustomVersion(FEditorObjectVersion::GUID);
	Ar.UsingCustomVersion(FReleaseObjectVersion::GUID);
	Ar.UsingCustomVersion(FRenderingObjectVersion::GUID);
	Ar.UsingCustomVersion(FAnimObjectVersion::GUID);
	Ar.UsingCustomVersion(FSkeletalMeshCustomVersion::GUID);
	Ar.UsingCustomVersion(FUE5MainStreamObjectVersion::GUID);
	Ar.UsingCustomVersion(FRecomputeTangentCustomVersion::GUID);
	Ar.UsingCustomVersion(FOverlappingVerticesCustomVersion::GUID);
	Ar.UsingCustomVersion(FUE5ReleaseStreamObjectVersion::GUID);
	Ar.UsingCustomVersion(FFortniteMainBranchObjectVersion::GUID);
}

// Serialization.
FArchive& operator<<(FArchive& Ar, FSkelMeshSourceSectionUserData& S)
{
	Ar.UsingCustomVersion(FUE5MainStreamObjectVersion::GUID);
	Ar.UsingCustomVersion(FRecomputeTangentCustomVersion::GUID); 

	FStripDataFlags StripFlags(Ar);
	// When data is cooked we do not serialize anything
	//This is for editor only editing
	if (StripFlags.IsEditorDataStripped())
	{
		return Ar;
	}

	Ar << S.bRecomputeTangent;
	if (Ar.CustomVer(FRecomputeTangentCustomVersion::GUID) >= FRecomputeTangentCustomVersion::RecomputeTangentVertexColorMask)
	{
		Ar << S.RecomputeTangentsVertexMaskChannel;
	}
 	else
	{
		// Our default is not to use vertex color as mask
		S.RecomputeTangentsVertexMaskChannel = ESkinVertexColorChannel::None;
	}

	Ar << S.bCastShadow;

	if (Ar.CustomVer(FUE5MainStreamObjectVersion::GUID) >= FUE5MainStreamObjectVersion::SkelMeshSectionVisibleInRayTracingFlagAdded)
	{
		Ar << S.bVisibleInRayTracing;
	}
	else
	{
		// default is to be visible in ray tracing - which is consistent with behaviour before adding this member
		S.bVisibleInRayTracing = true;
	}

	Ar << S.bDisabled;
	Ar << S.GenerateUpToLodIndex;
	Ar << S.CorrespondClothAssetIndex;
	Ar << S.ClothingData;

	return Ar;
}

//////////////////////////////////////////////////////////////////////////

/** Legacy Chunk struct, now merged with FSkelMeshSection */
struct FLegacySkelMeshChunk
{
	uint32 BaseVertexIndex;
	TArray<FSoftSkinVertex> SoftVertices;
	TArray<FMeshToMeshVertData> ApexClothMappingData;
	TArray<FVector> PhysicalMeshVertices;
	TArray<FVector> PhysicalMeshNormals;
	TArray<FBoneIndexType> BoneMap;
	int32 MaxBoneInfluences;

	int16 CorrespondClothAssetIndex;
	int16 ClothAssetSubmeshIndex;

	FLegacySkelMeshChunk()
		: BaseVertexIndex(0)
		, MaxBoneInfluences(4)
		, CorrespondClothAssetIndex(INDEX_NONE)
		, ClothAssetSubmeshIndex(INDEX_NONE)
	{}

	void CopyToSection(FSkelMeshSection& Section)
	{
		Section.BaseVertexIndex = BaseVertexIndex;
		Section.SoftVertices = SoftVertices;

		constexpr int32 ClothLODBias = 0;  // There isn't any cloth LOD bias on legacy sections
		Section.ClothMappingDataLODs.SetNum(1);
		Section.ClothMappingDataLODs[ClothLODBias] = ApexClothMappingData;

		Section.BoneMap = BoneMap;
		Section.MaxBoneInfluences = MaxBoneInfluences;
		Section.CorrespondClothAssetIndex = CorrespondClothAssetIndex;
	}


	friend FArchive& operator<<(FArchive& Ar, FLegacySkelMeshChunk& C)
	{
		FStripDataFlags StripFlags(Ar);

		if (!StripFlags.IsAudioVisualDataStripped())
		{
			// This is so that BaseVertexIndex is never set to anything else that 0 (for safety)
			Ar << C.BaseVertexIndex;
		}
		if (!StripFlags.IsEditorDataStripped())
		{
			// For backwards compat, read rigid vert array into array
			TArray<FLegacyRigidSkinVertex> LegacyRigidVertices;
			if (Ar.IsLoading() && Ar.CustomVer(FSkeletalMeshCustomVersion::GUID) < FSkeletalMeshCustomVersion::CombineSoftAndRigidVerts)
			{
				Ar << LegacyRigidVertices;
			}

			Ar << C.SoftVertices;

			// Once we have read in SoftVertices, convert and insert legacy rigid verts (if present) at start
			const int32 NumRigidVerts = LegacyRigidVertices.Num();
			if (NumRigidVerts > 0 && Ar.CustomVer(FSkeletalMeshCustomVersion::GUID) < FSkeletalMeshCustomVersion::CombineSoftAndRigidVerts)
			{
				C.SoftVertices.InsertUninitialized(0, NumRigidVerts);

				for (int32 VertIdx = 0; VertIdx < NumRigidVerts; VertIdx++)
				{
					LegacyRigidVertices[VertIdx].ConvertToSoftVert(C.SoftVertices[VertIdx]);
				}
			}
		}
		Ar << C.BoneMap;

		// Removed NumRigidVertices and NumSoftVertices, just use array size
		if (Ar.CustomVer(FSkeletalMeshCustomVersion::GUID) < FSkeletalMeshCustomVersion::CombineSoftAndRigidVerts)
		{
			int32 DummyNumRigidVerts, DummyNumSoftVerts;
			Ar << DummyNumRigidVerts;
			Ar << DummyNumSoftVerts;

			if (DummyNumRigidVerts + DummyNumSoftVerts != C.SoftVertices.Num())
			{
				UE_LOGF(LogSkeletalMesh, Error, "Legacy NumSoftVerts + NumRigidVerts != SoftVertices.Num()");
			}
		}

		Ar << C.MaxBoneInfluences;


		if (Ar.UEVer() >= VER_UE4_APEX_CLOTH)
		{
			Ar << C.ApexClothMappingData;
			Ar << C.PhysicalMeshVertices;
			Ar << C.PhysicalMeshNormals;
			Ar << C.CorrespondClothAssetIndex;
			Ar << C.ClothAssetSubmeshIndex;
		}

		return Ar;
	}
};

void FSkeletalMeshLODModel::Serialize(FArchive& Ar, UObject* Owner, int32 Idx)
{
	DECLARE_SCOPE_CYCLE_COUNTER(TEXT("FSkeletalMeshLODModel::Serialize"), STAT_SkeletalMeshLODModel_Serialize, STATGROUP_LoadTime);

	const uint8 LodAdjacencyStripFlag = 1;
	FStripDataFlags StripFlags(Ar, Ar.IsCooking() ? LodAdjacencyStripFlag : 0);

	Ar.UsingCustomVersion(FSkeletalMeshCustomVersion::GUID);
	Ar.UsingCustomVersion(FFortniteMainBranchObjectVersion::GUID);
	Ar.UsingCustomVersion(FEditorObjectVersion::GUID);
	Ar.UsingCustomVersion(FUE5MainStreamObjectVersion::GUID);
	Ar.UsingCustomVersion(FUE5ReleaseStreamObjectVersion::GUID);

	if (StripFlags.IsAudioVisualDataStripped())
	{
		TArray<FSkelMeshSection> TempSections;
		Ar << TempSections;

		if (Ar.CustomVer(FEditorObjectVersion::GUID) >= FEditorObjectVersion::SkeletalMeshBuildRefactor)
		{
			TMap<int32, FSkelMeshSourceSectionUserData> TempUserSectionsData;
			Ar << TempUserSectionsData;
		}

		// For old content, load as a multi-size container
		if (Ar.IsLoading() && Ar.CustomVer(FSkeletalMeshCustomVersion::GUID) < FSkeletalMeshCustomVersion::SplitModelAndRenderData)
		{
			FMultiSizeIndexContainer TempMultiSizeIndexContainer;
			TempMultiSizeIndexContainer.Serialize(Ar, false);
		}
		else
		{
			TArray<int32> DummyIndexBuffer;
			Ar << DummyIndexBuffer;
		}

		TArray<FBoneIndexType> TempActiveBoneIndices;
		Ar << TempActiveBoneIndices;

		if (Ar.CustomVer(FUE5MainStreamObjectVersion::GUID) >= FUE5MainStreamObjectVersion::SkeletalMeshLODModelMeshInfo)
		{
			TArray<FSkelMeshImportedMeshInfo> TempMeshInfos;
			Ar << TempMeshInfos;
		}
	}
	else
	{
		Ar << Sections;
		
		if (!StripFlags.IsEditorDataStripped() && Ar.CustomVer(FEditorObjectVersion::GUID) >= FEditorObjectVersion::SkeletalMeshBuildRefactor)
		{
			//Editor builds only
			Ar << UserSectionsData;
		}

		// For old content, load as a multi-size container, but convert into regular array
		if (Ar.IsLoading() && Ar.CustomVer(FSkeletalMeshCustomVersion::GUID) < FSkeletalMeshCustomVersion::SplitModelAndRenderData)
		{
			FMultiSizeIndexContainer TempMultiSizeIndexContainer;
			TempMultiSizeIndexContainer.Serialize(Ar, false);

			// Only save index buffer data in editor builds
			if (!StripFlags.IsEditorDataStripped())
			{
				TempMultiSizeIndexContainer.GetIndexBuffer(IndexBuffer);
			}
		}
		// Only load index buffer data in editor builds
		else if(!StripFlags.IsEditorDataStripped())
		{
			Ar << IndexBuffer;
		}

		Ar << ActiveBoneIndices;

		// Editor only data.
		if (!StripFlags.IsEditorDataStripped() && Ar.CustomVer(FUE5MainStreamObjectVersion::GUID) >= FUE5MainStreamObjectVersion::SkeletalMeshLODModelMeshInfo)
		{
			Ar << ImportedMeshInfos;
		}
	}

	// Array of Sections for backwards compat
	if (Ar.IsLoading() && Ar.CustomVer(FSkeletalMeshCustomVersion::GUID) < FSkeletalMeshCustomVersion::CombineSectionWithChunk)
	{
		TArray<FLegacySkelMeshChunk> LegacyChunks;

		Ar << LegacyChunks;

		check(LegacyChunks.Num() == Sections.Num());
		for (int32 ChunkIdx = 0; ChunkIdx < LegacyChunks.Num(); ChunkIdx++)
		{
			FSkelMeshSection& Section = Sections[ChunkIdx];

			LegacyChunks[ChunkIdx].CopyToSection(Section);

			// Set NumVertices for older content on load
			if (!StripFlags.IsAudioVisualDataStripped())
			{
				Section.NumVertices = Section.SoftVertices.Num();
			}
			else
			{
				UE_LOGF(LogSkeletalMesh, Warning, "Cannot set FSkelMeshSection::NumVertices for older content, loading in non-editor build.");
				Section.NumVertices = 0;
			}
		}
	}

	// no longer in use
	{
		uint32 LegacySize = 0;
		Ar << LegacySize;
	}

	if (!StripFlags.IsAudioVisualDataStripped())
	{
		Ar << NumVertices;
	}
	Ar << RequiredBones;

	if (!StripFlags.IsEditorDataStripped())
	{
		if (Ar.IsLoading() && Ar.CustomVer(FUE5ReleaseStreamObjectVersion::GUID) < FUE5ReleaseStreamObjectVersion::RemoveSkeletalMeshLODModelBulkDatas)
		{
			RawPointIndices_DEPRECATED.Serialize(Ar, Owner);
			if (RawPointIndices_DEPRECATED.GetBulkDataSize())
			{
				if (!RawPointIndices_DEPRECATED.IsBulkDataLoaded())
				{
					RawPointIndices_DEPRECATED.LoadBulkDataWithFileReader();
				}
			
				RawPointIndices2.Empty(RawPointIndices_DEPRECATED.GetElementCount());
				RawPointIndices2.AddUninitialized(RawPointIndices_DEPRECATED.GetElementCount());
				FMemory::Memcpy(RawPointIndices2.GetData(), RawPointIndices_DEPRECATED.Lock(LOCK_READ_ONLY), RawPointIndices_DEPRECATED.GetBulkDataSize());
				RawPointIndices_DEPRECATED.Unlock();
			}
			RawPointIndices_DEPRECATED.RemoveBulkData();
		}
		else
		{
			Ar << RawPointIndices2;
		}
		if (Ar.IsLoading()
			&& (Ar.CustomVer(FFortniteMainBranchObjectVersion::GUID) >= FFortniteMainBranchObjectVersion::NewSkeletalMeshImporterWorkflow)
			&& (Ar.CustomVer(FEditorObjectVersion::GUID) < FEditorObjectVersion::SkeletalMeshMoveEditorSourceDataToPrivateAsset))
		{
			RawSkeletalMeshBulkData_DEPRECATED.Serialize(Ar, Owner);
			RawSkeletalMeshBulkDataID = RawSkeletalMeshBulkData_DEPRECATED.GetIdString();
			bIsBuildDataAvailable = RawSkeletalMeshBulkData_DEPRECATED.IsBuildDataAvailable();
			bIsRawSkeletalMeshBulkDataEmpty = RawSkeletalMeshBulkData_DEPRECATED.IsEmpty();
		}
		if (Ar.CustomVer(FEditorObjectVersion::GUID) >= FEditorObjectVersion::SkeletalMeshMoveEditorSourceDataToPrivateAsset)
		{
			Ar << RawSkeletalMeshBulkDataID;
			Ar << bIsBuildDataAvailable;
			Ar << bIsRawSkeletalMeshBulkDataEmpty;
		}
	}

	if (StripFlags.IsAudioVisualDataStripped())
	{
		TArray<int32> TempMeshToImportVertexMap;
		Ar << TempMeshToImportVertexMap;

		int32 TempMaxImportVertex;
		Ar << TempMaxImportVertex;
	}
	else
	{
		Ar << MeshToImportVertexMap;
		Ar << MaxImportVertex;
	}

	if (!StripFlags.IsAudioVisualDataStripped())
	{
		Ar << NumTexCoords;

		// All this data has now moved to derived data, but need to handle loading older LOD Models where it was serialized with asset
		if (Ar.IsLoading() && Ar.CustomVer(FSkeletalMeshCustomVersion::GUID) < FSkeletalMeshCustomVersion::SplitModelAndRenderData)
		{
			FDummySkeletalMeshVertexBuffer DummyVertexBuffer;
			Ar << DummyVertexBuffer;

			if (Ar.CustomVer(FSkeletalMeshCustomVersion::GUID) >= FSkeletalMeshCustomVersion::UseSeparateSkinWeightBuffer)
			{
				FSkinWeightVertexBuffer DummyWeightBuffer;
				Ar << DummyWeightBuffer;
			}

			USkeletalMesh* SkelMeshOwner = CastChecked<USkeletalMesh>(Owner);
			if (SkelMeshOwner->GetHasVertexColors())
			{
				// Handling for old color buffer data
				if (Ar.IsLoading() && Ar.CustomVer(FSkeletalMeshCustomVersion::GUID) < FSkeletalMeshCustomVersion::UseSharedColorBufferFormat)
				{
					TArray<FColor> OldColors;
					FStripDataFlags LegacyColourStripFlags(Ar, 0, FPackageFileVersion::CreateUE4Version(VER_UE4_STATIC_SKELETAL_MESH_SERIALIZATION_FIX));
					OldColors.BulkSerialize(Ar);
				}
				else
				{
					FColorVertexBuffer DummyColorBuffer;
					DummyColorBuffer.Serialize(Ar, false);
					//Copy the data to the softVertices
					int32 VertexColorCount = DummyColorBuffer.GetNumVertices();
					if (NumVertices == VertexColorCount)
					{
						TArray<FColor> OutColors;
						DummyColorBuffer.GetVertexColors(OutColors);
						int32 DummyVertexColorIndex = 0;
						for (int32 SectionIndex = 0; SectionIndex < Sections.Num(); ++SectionIndex)
						{
							int32 SectionVertexCount = Sections[SectionIndex].GetNumVertices();
							TArray<FSoftSkinVertex>& SoftVertices = Sections[SectionIndex].SoftVertices;
							for (int32 SectionVertexIndex = 0; SectionVertexIndex < SectionVertexCount; ++SectionVertexIndex)
							{
								SoftVertices[SectionVertexIndex].Color = OutColors[DummyVertexColorIndex++];
							}
						}
					}
				}
			}

			if (!StripFlags.IsClassDataStripped(LodAdjacencyStripFlag))
			{
				// For old content, load as a multi-size container, but convert into regular array
				{
					// Serialize and discard the adjacency data, it's now build for the DDC
					FMultiSizeIndexContainer TempMultiSizeAdjacencyIndexContainer;
					TempMultiSizeAdjacencyIndexContainer.Serialize(Ar, false);
				}
			}

			if (Ar.UEVer() >= VER_UE4_APEX_CLOTH && HasClothData())
			{
				FStripDataFlags StripFlags2(Ar, 0, FPackageFileVersion::CreateUE4Version(VER_UE4_STATIC_SKELETAL_MESH_SERIALIZATION_FIX));
				TSkeletalMeshVertexData<FMeshToMeshVertData> DummyClothData(true);

				if (!StripFlags2.IsAudioVisualDataStripped() || Ar.IsCountingMemory())
				{
					DummyClothData.Serialize(Ar);
			
					if (Ar.CustomVer(FSkeletalMeshCustomVersion::GUID) >= FSkeletalMeshCustomVersion::CompactClothVertexBuffer)
					{
						TArray<uint64> DummyIndexMapping;
						Ar << DummyIndexMapping;
					}
				}
			}
		}
	}

	if (Ar.CustomVer(FSkeletalMeshCustomVersion::GUID) >= FSkeletalMeshCustomVersion::SkinWeightProfiles)
	{
		Ar << SkinWeightProfiles;
	}
}

void FSkeletalMeshLODModel::DeclareCustomVersions(FArchive& Ar)
{
	Ar.UsingCustomVersion(FSkeletalMeshCustomVersion::GUID);
	Ar.UsingCustomVersion(FFortniteMainBranchObjectVersion::GUID);
	Ar.UsingCustomVersion(FEditorObjectVersion::GUID);
	FSkelMeshSection::DeclareCustomVersions(Ar);
}

void FSkeletalMeshLODModel::GetSectionFromVertexIndex(int32 InVertIndex, int32& OutSectionIndex, int32& OutVertIndex) const
{
	OutSectionIndex = 0;
	OutVertIndex = 0;

	int32 VertCount = 0;

	// Iterate over each chunk
	for (int32 SectionCount = 0; SectionCount < Sections.Num(); SectionCount++)
	{
		const FSkelMeshSection& Section = Sections[SectionCount];
		OutSectionIndex = SectionCount;

		// Is it in Soft vertex range?
		if (InVertIndex < VertCount + Section.GetNumVertices())
		{
			OutVertIndex = InVertIndex - VertCount;
			return;
		}
		VertCount += Section.GetNumVertices();
	}

	// InVertIndex should always be in some chunk!
	//check(false);
}

void FSkeletalMeshLODModel::GetVertices(TArray<FSoftSkinVertex>& Vertices, const FSkeletalMeshUnifiedBoneIndices* UnifiedBoneIndices) const
{
	Vertices.Empty(NumVertices);
	Vertices.AddUninitialized(NumVertices);

	// validate NumVertices is correct
	{
		int32 TotalSoftVertices = 0;
		for (int32 SectionIndex = 0; SectionIndex < Sections.Num(); SectionIndex++)
		{
			TotalSoftVertices += Sections[SectionIndex].SoftVertices.Num();
		}
		if (TotalSoftVertices != NumVertices)
		{
			// hitting this means NumVertices didn't match the sum of the vertex counts of all the sections,
			// which could potentially overrun the Vertices buffer's allocation
			UE_LOGF(LogSkeletalMesh, Fatal, "NumVertices (%i) != TotalSoftVertices (%i)", NumVertices, TotalSoftVertices);
		}
	}

	// Initialize the vertex data
	// All chunks are combined into one (rigid first, soft next)
	FSoftSkinVertex* DstVertices = (FSoftSkinVertex*)Vertices.GetData();

	for (const FSkelMeshSection& Section : Sections)
	{
		if (UnifiedBoneIndices && UnifiedBoneIndices->IsValid())
		{
			for (int32 VertexIndex = 0; VertexIndex < Section.SoftVertices.Num(); ++VertexIndex)
			{
				FSoftSkinVertex& DstVertex = DstVertices[VertexIndex];
				DstVertex = Section.SoftVertices[VertexIndex];
				UnifiedBoneIndices->RemapVertex(Section.BoneMap, DstVertex);
			}
		}
		else
		{
			FMemory::Memcpy(DstVertices, Section.SoftVertices.GetData(), Section.SoftVertices.Num() * sizeof(FSoftSkinVertex));
		}

		DstVertices += Section.SoftVertices.Num();
	}
}

void FSkeletalMeshLODModel::GetClothMappingData(TArray<FMeshToMeshVertData>& MappingData, TArray<FClothBufferIndexMapping>& OutClothIndexMapping) const
{
	for (int32 SectionIndex = 0; SectionIndex < Sections.Num(); SectionIndex++)
	{
		const FSkelMeshSection& Section = Sections[SectionIndex];
		constexpr int32 ClothLODBias = 0;  // Use the default cloth LOD bias of 0 for calculations, this means the same LOD as the current section
		if (Section.ClothMappingDataLODs.Num() && Section.ClothMappingDataLODs[ClothLODBias].Num())
		{
			FClothBufferIndexMapping ClothBufferIndexMapping;
			ClothBufferIndexMapping.BaseVertexIndex = Section.BaseVertexIndex;
			ClothBufferIndexMapping.MappingOffset = (uint32)MappingData.Num();
			ClothBufferIndexMapping.LODBiasStride = (uint32)Section.ClothMappingDataLODs[ClothLODBias].Num();

			OutClothIndexMapping.Add(ClothBufferIndexMapping);

			// Append all mapping LODs to the output array for this section
			for (const TArray<FMeshToMeshVertData>& ClothMappingDataLOD : Section.ClothMappingDataLODs)
			{
				MappingData += ClothMappingDataLOD;
			}
		}
		else
		{
			OutClothIndexMapping.Add({ 0, 0, 0 });
		}
	}
}

void FSkeletalMeshLODModel::GetResourceSizeEx(FResourceSizeEx& CumulativeResourceSize) const
{
	CumulativeResourceSize.AddUnknownMemoryBytes(Sections.GetAllocatedSize());
	CumulativeResourceSize.AddUnknownMemoryBytes(ActiveBoneIndices.GetAllocatedSize());
	CumulativeResourceSize.AddUnknownMemoryBytes(RequiredBones.GetAllocatedSize());
	CumulativeResourceSize.AddUnknownMemoryBytes(IndexBuffer.GetAllocatedSize());

	CumulativeResourceSize.AddUnknownMemoryBytes(RawPointIndices2.GetAllocatedSize());
	CumulativeResourceSize.AddUnknownMemoryBytes(MeshToImportVertexMap.GetAllocatedSize());
}

bool FSkeletalMeshLODModel::HasClothData() const
{
	for (int32 SectionIdx = 0; SectionIdx < Sections.Num(); SectionIdx++)
	{
		if (Sections[SectionIdx].HasClothingData())
		{
			return true;
		}
	}
	return false;
}

int32 FSkeletalMeshLODModel::NumNonClothingSections() const
{
	const int32 NumSections = Sections.Num();
	int32 Count = 0;

	for (int32 SectionIndex = 0; SectionIndex < NumSections; SectionIndex++)
	{
		if(!Sections[SectionIndex].HasClothingData())
		{
			Count++;
		}
	}

	return Count;
}

int32 FSkeletalMeshLODModel::GetNumNonClothingVertices() const
{
	int32 NumVerts = 0;
	int32 NumSections = Sections.Num();

	for (int32 i = 0; i < NumSections; i++)
	{
		const FSkelMeshSection& Section = Sections[i];

		// Stop when we hit clothing sections
		if (Section.ClothingData.AssetGuid.IsValid())
		{
			continue;
		}

		NumVerts += Section.SoftVertices.Num();
	}

	return NumVerts;
}

void FSkeletalMeshLODModel::GetNonClothVertices(TArray<FSoftSkinVertex>& OutVertices, const FSkeletalMeshUnifiedBoneIndices* UnifiedBoneIndices) const
{
	// Get the number of sections to copy
	int32 NumSections = Sections.Num();

	// Count number of verts
	int32 NumVertsToCopy = GetNumNonClothingVertices();

	OutVertices.Empty(NumVertsToCopy);
	OutVertices.AddUninitialized(NumVertsToCopy);

	// Initialize the vertex data
	// All chunks are combined into one (rigid first, soft next)
	FSoftSkinVertex* DstVertices = (FSoftSkinVertex*)OutVertices.GetData();
	for (const FSkelMeshSection& Section : Sections)
	{
		if (Section.ClothingData.AssetGuid.IsValid())
		{
			continue;
		}

		if (UnifiedBoneIndices && UnifiedBoneIndices->IsValid())
		{
			for (int32 VertexIndex = 0; VertexIndex < Section.SoftVertices.Num(); ++VertexIndex)
			{
				FSoftSkinVertex& DstVertex = DstVertices[VertexIndex];
				DstVertex = Section.SoftVertices[VertexIndex];
				UnifiedBoneIndices->RemapVertex(Section.BoneMap, DstVertex);
			}
		}
		else
		{
			FMemory::Memcpy(DstVertices, Section.SoftVertices.GetData(), Section.SoftVertices.Num() * sizeof(FSoftSkinVertex));
		}

		DstVertices += Section.SoftVertices.Num();
	}
}

int32 FSkeletalMeshLODModel::GetMaxBoneInfluences() const
{
	int32 NumBoneInfluences = 0;
	for (int32 SectionIdx = 0; SectionIdx < Sections.Num(); ++SectionIdx)
	{
		NumBoneInfluences = FMath::Max(NumBoneInfluences, Sections[SectionIdx].GetMaxBoneInfluences());
	}

	return NumBoneInfluences;
}

bool FSkeletalMeshLODModel::DoSectionsUse16BitBoneIndex() const
{
	for (int32 SectionIdx = 0; SectionIdx < Sections.Num(); ++SectionIdx)
	{
		if (Sections[SectionIdx].Use16BitBoneIndex())
		{
			return true;
		}
	}

	return false;
}

void FSkeletalMeshLODModel::SyncronizeUserSectionsDataArray(bool bResetNonUsedSection /*= false*/)
{
	int32 SectionNum = Sections.Num();
	for (int32 SectionIndex = 0; SectionIndex < SectionNum; ++SectionIndex)
	{
		FSkelMeshSection& Section = Sections[SectionIndex];
		FSkelMeshSourceSectionUserData& SectionUserData = UserSectionsData.FindOrAdd(Section.OriginalDataSectionIndex);
		Section.bCastShadow					= SectionUserData.bCastShadow;
		Section.bVisibleInRayTracing		= SectionUserData.bVisibleInRayTracing;
		Section.bRecomputeTangent			= SectionUserData.bRecomputeTangent;
		Section.RecomputeTangentsVertexMaskChannel = SectionUserData.RecomputeTangentsVertexMaskChannel;
		Section.bDisabled					= SectionUserData.bDisabled;
		Section.GenerateUpToLodIndex		= SectionUserData.GenerateUpToLodIndex;
		Section.CorrespondClothAssetIndex	= SectionUserData.CorrespondClothAssetIndex;
		Section.ClothingData.AssetGuid		= SectionUserData.ClothingData.AssetGuid;
		Section.ClothingData.AssetLodIndex	= SectionUserData.ClothingData.AssetLodIndex;
	}

	//Reset normally happen when we re-import a skeletalmesh, we never want to reset this when we build the skeletalmesh (reduce can remove section, but we need to keep the original section data)
	if (bResetNonUsedSection)
	{
		//Make sure we have the correct amount of UserSectionData we delete all the entries and recreate them with the previously sync Sections
		UserSectionsData.Reset();
		for (int32 SectionIndex = 0; SectionIndex < SectionNum; ++SectionIndex)
		{
			FSkelMeshSection& Section = Sections[SectionIndex];
			//We only need parent section, no need to iterate bone chunked sections
			if (Section.ChunkedParentSectionIndex != INDEX_NONE)
			{
				continue;
			}
			FSkelMeshSourceSectionUserData& SectionUserData = UserSectionsData.FindOrAdd(Section.OriginalDataSectionIndex);
			SectionUserData.bCastShadow = Section.bCastShadow;
			SectionUserData.bVisibleInRayTracing = Section.bVisibleInRayTracing;
			SectionUserData.bRecomputeTangent = Section.bRecomputeTangent;
			SectionUserData.RecomputeTangentsVertexMaskChannel = Section.RecomputeTangentsVertexMaskChannel;
			SectionUserData.bDisabled = Section.bDisabled;
			SectionUserData.GenerateUpToLodIndex = Section.GenerateUpToLodIndex;
			SectionUserData.CorrespondClothAssetIndex = Section.CorrespondClothAssetIndex;
			SectionUserData.ClothingData.AssetGuid = Section.ClothingData.AssetGuid;
			SectionUserData.ClothingData.AssetLodIndex = Section.ClothingData.AssetLodIndex;
		}
	}
}

FString FSkeletalMeshLODModel::GetLODModelDeriveDataKey() const
{
	FString KeySuffix = "LODMODEL";

	TArray<uint8> ByteData;
	FMemoryWriter Ar(ByteData, true);

	//Add the bulk data ID (if someone modify the original imported data, this ID will change)
	FString BulkDatIDString = RawSkeletalMeshBulkDataID; //Need to re-assign to tmp var because function is const
	Ar << BulkDatIDString;
	int32 UserSectionCount = UserSectionsData.Num();
	Ar << UserSectionCount;
	for (auto Kvp : UserSectionsData)
	{
		Ar << Kvp.Key;
		Ar << Kvp.Value;
	}

	FSHA1 Sha;
	Sha.Update(ByteData.GetData(), ByteData.Num() * ByteData.GetTypeSize());
	Sha.Final();
	// Retrieve the hash and use it to construct a pseudo-GUID.
	uint32 Hash[5];
	Sha.GetHash((uint8*)Hash);
	KeySuffix += FGuid(Hash[0] ^ Hash[4], Hash[1], Hash[2], Hash[3]).ToString(EGuidFormats::Digits);

	return KeySuffix;
}

void FSkeletalMeshLODModel::UpdateChunkedSectionInfo(const FString& SkeletalMeshName)
{
	int32 LODModelSectionNum = Sections.Num();
	//Fill the ChunkedParentSectionIndex data, we assume that every section using the same material are chunked
	int32 LastMaterialIndex = INDEX_NONE;
	int32 LastOriginalDataSectionIndex = INDEX_NONE;
	int32 CurrentParentChunkIndex = INDEX_NONE;
	int32 OriginalIndex = 0;

	for (int32 LODModelSectionIndex = 0; LODModelSectionIndex < LODModelSectionNum; ++LODModelSectionIndex)
	{
		FSkelMeshSection& Section = Sections[LODModelSectionIndex];
		
		//If we have already chunked data in this LODModel use it to know if we need to chunk a section or not, this can happen when we load reduction data.
		const bool bIsOldChunkingSection = LastOriginalDataSectionIndex != INDEX_NONE && Section.OriginalDataSectionIndex == LastOriginalDataSectionIndex;
		//If we have cloth on a chunked section we treat the chunked section has a parent section (this is to get the same result has before the refactor)
		if (bIsOldChunkingSection && Section.MaterialIndex == LastMaterialIndex && !Section.ClothingData.AssetGuid.IsValid())
		{
			Section.ChunkedParentSectionIndex = CurrentParentChunkIndex;
			Section.OriginalDataSectionIndex = Sections[CurrentParentChunkIndex].OriginalDataSectionIndex;
			//In case of a child section that was BONE chunked ensure it has the same setting has the original section
			FSkelMeshSourceSectionUserData& SectionUserData = UserSectionsData.FindOrAdd(Section.OriginalDataSectionIndex);
			Section.bDisabled = SectionUserData.bDisabled;
			Section.bCastShadow = SectionUserData.bCastShadow;
			Section.bVisibleInRayTracing = SectionUserData.bVisibleInRayTracing;
			Section.bRecomputeTangent = SectionUserData.bRecomputeTangent;
			Section.RecomputeTangentsVertexMaskChannel = SectionUserData.RecomputeTangentsVertexMaskChannel;
			Section.GenerateUpToLodIndex = SectionUserData.GenerateUpToLodIndex;
			//Chunked section cannot have cloth, a cloth section will be a parent section
			Section.CorrespondClothAssetIndex = INDEX_NONE;
			Section.ClothingData.AssetGuid = FGuid();
			Section.ClothingData.AssetLodIndex = INDEX_NONE;
		}
		else
		{
			CurrentParentChunkIndex = LODModelSectionIndex;
			FSkelMeshSourceSectionUserData& SectionUserData = UserSectionsData.FindOrAdd(OriginalIndex);
			SectionUserData.bDisabled = Section.bDisabled;
			SectionUserData.bCastShadow = Section.bCastShadow;
			SectionUserData.bVisibleInRayTracing = Section.bVisibleInRayTracing;
			SectionUserData.bRecomputeTangent = Section.bRecomputeTangent;
			SectionUserData.RecomputeTangentsVertexMaskChannel = Section.RecomputeTangentsVertexMaskChannel;
			SectionUserData.GenerateUpToLodIndex = Section.GenerateUpToLodIndex;
			//Make sure the CorrespondClothAssetIndex is valid
			if (Section.CorrespondClothAssetIndex < -1)
			{
				Section.CorrespondClothAssetIndex = INDEX_NONE;
			}

			SectionUserData.CorrespondClothAssetIndex = Section.CorrespondClothAssetIndex;
			SectionUserData.ClothingData.AssetGuid = Section.ClothingData.AssetGuid;
			SectionUserData.ClothingData.AssetLodIndex = Section.ClothingData.AssetLodIndex;

			Section.OriginalDataSectionIndex = OriginalIndex++;
			Section.ChunkedParentSectionIndex = INDEX_NONE;
		}

		LastMaterialIndex = Section.MaterialIndex;
		LastOriginalDataSectionIndex = Section.OriginalDataSectionIndex;
	}
}

void FSkeletalMeshLODModel::CopyStructure(FSkeletalMeshLODModel* Destination, const FSkeletalMeshLODModel* Source)
{
	//The private Lock should always be valid
	check(Source);
	check(Destination);
	check(Source->BulkDataReadMutex);
	check(Destination->BulkDataReadMutex);
	//Lock both mutex before touching the bulk data
	FScopeLock LockSource(Source->BulkDataReadMutex);
	FScopeLock LockDestination(Destination->BulkDataReadMutex);

	FCriticalSection* DestinationBulkDataReadMutex = Destination->BulkDataReadMutex;

	*Destination = *Source;

	//Make sure the mutex of the copy is set back to the original destination mutex, we can recycle the pointer.
	Destination->BulkDataReadMutex = DestinationBulkDataReadMutex;
}


void FSkeletalMeshLODModel::GetMeshDescription(const USkeletalMesh *InSkeletalMesh, const int32 InLODIndex, FMeshDescription& OutMeshDescription) const
{
	using UE::AnimationCore::FBoneWeights;

	OutMeshDescription.Empty();
	
	FSkeletalMeshAttributes MeshAttributes(OutMeshDescription);	
	
	// Register extra attributes for us.
	MeshAttributes.Register();

	TVertexAttributesRef<FVector3f> VertexPositions = MeshAttributes.GetVertexPositions();
	FSkinWeightsVertexAttributesRef VertexSkinWeights = MeshAttributes.GetVertexSkinWeights();
	TVertexInstanceAttributesRef<FVector3f> VertexInstanceNormals = MeshAttributes.GetVertexInstanceNormals();
	TVertexInstanceAttributesRef<FVector3f> VertexInstanceTangents = MeshAttributes.GetVertexInstanceTangents();
	TVertexInstanceAttributesRef<float> VertexInstanceBinormalSigns = MeshAttributes.GetVertexInstanceBinormalSigns();
	TVertexInstanceAttributesRef<FVector4f> VertexInstanceColors = MeshAttributes.GetVertexInstanceColors();
	TVertexInstanceAttributesRef<FVector2f> VertexInstanceUVs = MeshAttributes.GetVertexInstanceUVs();

	TPolygonGroupAttributesRef<FName> PolygonGroupMaterialSlotNames = MeshAttributes.GetPolygonGroupMaterialSlotNames();
	
	FSkeletalMeshAttributes::FBoneNameAttributesRef BoneNames = MeshAttributes.GetBoneNames();
	FSkeletalMeshAttributes::FBoneParentIndexAttributesRef BoneParentIndices = MeshAttributes.GetBoneParentIndices();
	FSkeletalMeshAttributes::FBonePoseAttributesRef BonePoses = MeshAttributes.GetBonePoses();

	// If the RawPointIndices2 is a map from the IndexBuffer to the original import vertices,
	// this most likely came from a USD/Alembic import where the normals are set explicitly.
	const bool bMorphTargetIncludeNormals = (RawPointIndices2.Num() == IndexBuffer.Num() && MeshToImportVertexMap.IsEmpty());
	
	TArray<TPair<FName, const FMorphTargetLODModel*>> MorphTargets;
	for (UMorphTarget* MorphTargetSource: InSkeletalMesh->GetMorphTargets())
	{
		if (!MorphTargetSource->HasDataForLOD(InLODIndex))
		{
			continue;
		}

		FName Name = MorphTargetSource->GetFName();
		if (Name.IsNone())
		{
			Name = TEXT("Unnamed");
		}
		
		MorphTargets.Emplace(Name, &MorphTargetSource->GetMorphLODModels()[InLODIndex]);
		MeshAttributes.RegisterMorphTargetAttribute(Name, bMorphTargetIncludeNormals);
	}

	for (const TPair<FName, FImportedSkinWeightProfileData>& SkinWeightProfileInfo: SkinWeightProfiles)
	{
		MeshAttributes.RegisterSkinWeightAttribute(SkinWeightProfileInfo.Key);
	}

	const int32 NumTriangles = IndexBuffer.Num() / 3;

	const FReferenceSkeleton& RefSkeleton = InSkeletalMesh->GetRefSkeleton();
	const int NumBones = RefSkeleton.GetRawBoneNum();

	OutMeshDescription.ReserveNewPolygonGroups(Sections.Num());
	OutMeshDescription.ReserveNewPolygons(NumTriangles);
	OutMeshDescription.ReserveNewTriangles(NumTriangles);
	OutMeshDescription.ReserveNewVertexInstances(NumTriangles * 3);
	OutMeshDescription.ReserveNewVertices(static_cast<int32>(NumVertices));
	MeshAttributes.ReserveNewBones(NumBones);

	// Map the section vertices back to the import vertices to remove seams, but only if there's
	// mapping available.
	TArray<int32> SourceToTargetVertexMap; 

	int32 TargetVertexCount = 0;
	
	if (RawPointIndices2.Num() == NumVertices)
	{
		SourceToTargetVertexMap.Reserve(RawPointIndices2.Num());
		
		for (const uint32 VertexIndex: RawPointIndices2)
		{
			SourceToTargetVertexMap.Add(VertexIndex);
			TargetVertexCount = FMath::Max(TargetVertexCount, static_cast<int32>(VertexIndex));
		}

		TargetVertexCount += 1;
	}
	else
	{
		SourceToTargetVertexMap.Reserve(NumVertices);
		for (uint32 Index = 0; Index < NumVertices; Index++)
		{
			SourceToTargetVertexMap.Add(Index);
		}
		TargetVertexCount = NumVertices;
	}
	
	TArray<FVertexID> VertexIDs;
	VertexIDs.Reserve(TargetVertexCount);
	for (int32 VertexIndex = 0; VertexIndex < TargetVertexCount; VertexIndex++)
	{
		VertexIDs.Add(OutMeshDescription.CreateVertex());
	}

	// Mapping to go from morph target vertices to vertex instances.
	TMultiMap<int32, FVertexInstanceID> SourceVertexToVertexInstanceMap;
	if (bMorphTargetIncludeNormals)
	{
		SourceVertexToVertexInstanceMap.Reserve(IndexBuffer.Num());
	}

	// Ensure we have enough channels to store all the defined UV coordinates.
	VertexInstanceUVs.SetNumChannels(static_cast<int32>(NumTexCoords));
	OutMeshDescription.SetNumUVChannels(static_cast<int32>(NumTexCoords));
	
	const TArray<FSkeletalMaterial>& Materials = InSkeletalMesh->GetMaterials();
	const bool bHasVertexColors = EnumHasAllFlags(InSkeletalMesh->GetVertexBufferFlags(), ESkeletalMeshVertexFlags::HasVertexColors);

	TSet<int32> ProcessedTargetVertex;
	ProcessedTargetVertex.Reserve(TargetVertexCount);
	
	// Convert sections to polygon groups, each with their own material.
	for (int32 SectionIndex = 0; SectionIndex < Sections.Num(); SectionIndex++)
	{
		const FSkelMeshSection& Section = Sections[SectionIndex];

		// Convert positions and bone weights
		const TArray<FSoftSkinVertex>& SourceVertices = Section.SoftVertices;
		for (int32 VertexIndex = 0; VertexIndex < SourceVertices.Num(); VertexIndex++)
		{
			const int32 SourceVertexIndex = VertexIndex + Section.BaseVertexIndex;
			const int32 TargetVertexIndex = SourceToTargetVertexMap[SourceVertexIndex];

			if (ProcessedTargetVertex.Contains(TargetVertexIndex))
			{
				continue;
			}
			ProcessedTargetVertex.Add(TargetVertexIndex);
			
			const FVertexID VertexID = VertexIDs[TargetVertexIndex];

			VertexPositions.Set(VertexID, SourceVertices[VertexIndex].Position);

			// Skeleton bone indexes translated from the render mesh compact indexes.
			FBoneIndexType	InfluenceBones[MAX_TOTAL_INFLUENCES];

			for (int32 InfluenceIndex = 0; InfluenceIndex < MAX_TOTAL_INFLUENCES && SourceVertices[VertexIndex].InfluenceWeights[InfluenceIndex]; InfluenceIndex++)
			{
				const int32 BoneId = SourceVertices[VertexIndex].InfluenceBones[InfluenceIndex];

				InfluenceBones[InfluenceIndex] = Section.BoneMap[BoneId];
			}

			VertexSkinWeights.Set(VertexID, FBoneWeights::Create(InfluenceBones, SourceVertices[VertexIndex].InfluenceWeights));
		}


		const FPolygonGroupID PolygonGroupID(Section.MaterialIndex);

		if (!OutMeshDescription.IsPolygonGroupValid(PolygonGroupID))
		{
			OutMeshDescription.CreatePolygonGroupWithID(PolygonGroupID);
		}

		if (ensure(Materials.IsValidIndex(Section.MaterialIndex)))
		{
			PolygonGroupMaterialSlotNames.Set(PolygonGroupID, Materials[Section.MaterialIndex].ImportedMaterialSlotName);
		}

		for (int32 TriangleID = 0; TriangleID < int32(Section.NumTriangles); TriangleID++)
		{
			const int32 VertexIndexBase = TriangleID * 3 + Section.BaseIndex;

			TArray<FVertexInstanceID> TriangleVertexInstanceIDs;
			TriangleVertexInstanceIDs.SetNum(3);

			for (int32 Corner = 0; Corner < 3; Corner++)
			{
				const int32 SourceVertexIndex = IndexBuffer[VertexIndexBase + Corner];
				const int32 TargetVertexIndex = SourceToTargetVertexMap[SourceVertexIndex];
				
				const FVertexID VertexID = VertexIDs[TargetVertexIndex];
				const FVertexInstanceID VertexInstanceID = OutMeshDescription.CreateVertexInstance(VertexID);

				if (bMorphTargetIncludeNormals)
				{
					SourceVertexToVertexInstanceMap.Add(SourceVertexIndex, VertexInstanceID);
				}

				const FSoftSkinVertex& SourceVertex = SourceVertices[SourceVertexIndex - Section.BaseVertexIndex];

				VertexInstanceNormals.Set(VertexInstanceID, SourceVertex.TangentZ);
				VertexInstanceTangents.Set(VertexInstanceID, SourceVertex.TangentX);
				VertexInstanceBinormalSigns.Set(VertexInstanceID, FMatrix44f(
					SourceVertex.TangentX.GetSafeNormal(),
					SourceVertex.TangentY.GetSafeNormal(),
					FVector3f(SourceVertex.TangentZ.GetSafeNormal()),
					FVector3f::ZeroVector).Determinant() < 0.0f ? -1.0f : +1.0f);

				for (int32 UVIndex = 0; UVIndex < static_cast<int32>(NumTexCoords); UVIndex++)
				{
					VertexInstanceUVs.Set(VertexInstanceID, UVIndex, SourceVertex.UVs[UVIndex]);
				}

				if (bHasVertexColors)
				{
					VertexInstanceColors.Set(VertexInstanceID, FVector4f(SourceVertex.Color.ReinterpretAsLinear()));
				}

				TriangleVertexInstanceIDs[Corner] = VertexInstanceID;
			}

			OutMeshDescription.CreateTriangle(PolygonGroupID, TriangleVertexInstanceIDs);
		}
	}

	// Copy morph targets.
	for (TPair<FName, const FMorphTargetLODModel*>& MorphSource: MorphTargets)
	{
		TVertexAttributesRef<FVector3f> PositionDelta = MeshAttributes.GetVertexMorphPositionDelta(MorphSource.Key);
		TVertexInstanceAttributesRef<FVector3f> NormalDelta = MeshAttributes.GetVertexInstanceMorphNormalDelta(MorphSource.Key);
		TArray<FVertexInstanceID> VertexInstanceIDs;

		for (const FMorphTargetDelta& Delta: MorphSource.Value->Vertices)
		{
			if (SourceToTargetVertexMap.IsValidIndex(Delta.SourceIdx))
			{
				const int32 TargetVertexIndex = SourceToTargetVertexMap[Delta.SourceIdx];
				const FVertexID VertexID = VertexIDs[TargetVertexIndex];

				PositionDelta.Set(VertexID, Delta.PositionDelta);

				if (bMorphTargetIncludeNormals)
				{
					VertexInstanceIDs.Reset();
					SourceVertexToVertexInstanceMap.MultiFind(Delta.SourceIdx, VertexInstanceIDs);

					for (FVertexInstanceID VertexInstanceID: VertexInstanceIDs)
					{
						NormalDelta.Set(VertexInstanceID, Delta.TangentZDelta); 
					}
				}
			}
		}
	}

	for (const TPair<FName, FImportedSkinWeightProfileData>& SkinWeightProfileInfo: SkinWeightProfiles)
	{
		FSkinWeightsVertexAttributesRef SkinWeightAttribute = MeshAttributes.GetVertexSkinWeights(SkinWeightProfileInfo.Key);
		const FImportedSkinWeightProfileData& SkinWeightProfileData = SkinWeightProfileInfo.Value;

#if 0
		// This version reads the copied import data influences. This is less precise than the ground truth version below
		// which is what will actually be used at render time.
		TMultiMap<int32, int32> InfluenceMap;
		for (int32 Index = 0; Index < SkinWeightProfileData.SourceModelInfluences.Num(); Index++)
		{
			const SkeletalMeshImportData::FVertInfluence& Influence = SkinWeightProfileData.SourceModelInfluences[Index];
			InfluenceMap.Add(Influence.VertIndex, Index);
		}

		TArray<FBoneIndexType> BoneIndexes;
		TArray<float> BoneWeights;
		TArray<int32> InfluenceIndexes;
		for (int32 Index = 0; Index < VertexIDs.Num(); Index++)
		{
			InfluenceIndexes.Reset();
			InfluenceMap.MultiFind(Index, InfluenceIndexes);

			if (!InfluenceIndexes.IsEmpty())
			{
				BoneIndexes.Reset();
				BoneWeights.Reset();
				for (int32 InfluenceIndex: InfluenceIndexes)
				{
					const SkeletalMeshImportData::FVertInfluence& Influence = SkinWeightProfileData.SourceModelInfluences[InfluenceIndex];
					BoneIndexes.Add(Influence.BoneIndex);
					BoneWeights.Add(Influence.Weight);
				}

				FBoneWeights Weights = FBoneWeights::Create(BoneIndexes.GetData(), BoneWeights.GetData(), BoneIndexes.Num());
				SkinWeightAttribute.Set(VertexIDs[Index], Weights);
			}
		}
#else
		check(SkinWeightProfileData.SkinWeights.Num() == NumVertices);

		int32 Index = 0;
		for (int32 SectionIndex = 0; SectionIndex < Sections.Num(); SectionIndex++)
		{
			const FSkelMeshSection& Section = Sections[SectionIndex];

			// Convert positions and bone weights
			const TArray<FBoneIndexType>& BoneMap = Section.BoneMap;
			for (int32 VertexIndex = 0; VertexIndex < Section.SoftVertices.Num(); VertexIndex++)
			{
				FRawSkinWeight RawSkinWeights = SkinWeightProfileData.SkinWeights[Index];
				const int32 TargetVertexIndex = SourceToTargetVertexMap[Index];
				const FVertexID VertexID = VertexIDs[TargetVertexIndex];

				for (int32 WeightIndex = 0; WeightIndex < MAX_TOTAL_INFLUENCES; WeightIndex++)
				{
					const uint16 BoneWeight = RawSkinWeights.InfluenceWeights[WeightIndex];
					FBoneIndexType& BoneIndex = RawSkinWeights.InfluenceBones[WeightIndex];
					if (BoneWeight != 0 && ensure(BoneMap.IsValidIndex(BoneIndex)))
					{
						BoneIndex = BoneMap[BoneIndex];
					}
				}
			
				FBoneWeights Weights = FBoneWeights::Create(RawSkinWeights.InfluenceBones, RawSkinWeights.InfluenceWeights);
				SkinWeightAttribute.Set(VertexID, Weights);

				Index++;
			}
		}
#endif
	}
	
	// Set Bone Attributes
	for (int Index = 0; Index < NumBones; ++Index)
	{
		const FMeshBoneInfo& BoneInfo = RefSkeleton.GetRawRefBoneInfo()[Index];
		const FTransform& BoneTransform = RefSkeleton.GetRawRefBonePose()[Index];

		const FBoneID BoneID = MeshAttributes.CreateBone();
		
		BoneNames.Set(BoneID, BoneInfo.Name);
		BoneParentIndices.Set(BoneID, BoneInfo.ParentIndex);
		BonePoses.Set(BoneID, BoneTransform);
	}
}

#endif // WITH_EDITOR
