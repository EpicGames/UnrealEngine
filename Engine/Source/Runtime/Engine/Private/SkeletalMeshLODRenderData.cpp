// Copyright Epic Games, Inc. All Rights Reserved.

#include "Rendering/SkeletalMeshLODRenderData.h"
#include "RenderUtils.h"
#include "Rendering/SkeletalMeshRenderData.h"
#include "Engine/SkinnedAsset.h"
#include "Engine/SkinnedAssetCommon.h"
#include "Animation/MeshDeformerProvider.h"
#include "Animation/MorphTarget.h"
#include "EngineLogs.h"
#include "EngineUtils.h"
#include "Interfaces/ITargetPlatform.h"
#include "Interfaces/ITargetPlatformManagerModule.h"
#include "Rendering/SkeletalMeshLODModel.h"
#include "Rendering/RenderCommandPipes.h"
#include "Serialization/MemoryReader.h"
#include "SkeletalMeshLegacyCustomVersions.h"
#include "UObject/Package.h"
#include "UObject/UE5MainStreamObjectVersion.h"
#include "UObject/UE5SpecialProjectStreamObjectVersion.h"
#include "GPUSkinCache.h"
#include "Serialization/MemoryWriter.h"
#include "RHIResourceUtils.h"

#if WITH_EDITOR
#include "Modules/ModuleManager.h"
#include "MeshUtilities.h"
#endif // WITH_EDITOR

int32 GStripSkeletalMeshLodsDuringCooking = 0;
static FAutoConsoleVariableRef CVarStripSkeletalMeshLodsBelowMinLod(
	TEXT("r.SkeletalMesh.StripMinLodDataDuringCooking"),
	GStripSkeletalMeshLodsDuringCooking,
	TEXT("If set will strip skeletal mesh LODs under the minimum renderable LOD for the target platform during cooking.")
);


namespace
{
	struct FReverseOrderBitArraysBySetBits
	{
		FORCEINLINE bool operator()(const TBitArray<>& Lhs, const TBitArray<>& Rhs) const
		{
			//sort by length
			if (Lhs.Num() != Rhs.Num())
			{
				return Lhs.Num() > Rhs.Num();
			}

			uint32 NumWords = FMath::DivideAndRoundUp(Lhs.Num(), NumBitsPerDWORD);
			const uint32* Data0 = Lhs.GetData();
			const uint32* Data1 = Rhs.GetData();

			//sort by num bits active
			int32 Count0 = 0, Count1 = 0;
			for (uint32 i = 0; i < NumWords; i++)
			{
				Count0 += FPlatformMath::CountBits(Data0[i]);
				Count1 += FPlatformMath::CountBits(Data1[i]);
			}

			if (Count0 != Count1)
			{
				return Count0 > Count1;
			}

			//sort by big-num value
			for (uint32 i = NumWords - 1; i != ~0u; i--)
			{
				if (Data0[i] != Data1[i])
				{
					return Data0[i] > Data1[i];
				}
			}
			return false;
		}
	};
}

/** 
 * This function returns true if the duplicate vertices should be cooked. 
 * The data is used to deal with seams along split vertices when recomputing normals at runtime.
 */
static bool RequiresDuplicateVerticesInCook(EShaderPlatform InPlatform, FSkelMeshRenderSection const& RenderSection)
{
	// DuplicatedVertices are cooked if we have GPUSkinCache and r.SkinCache.StoreDuplicatedVerticesForRecomputeTangents is set or if the section was marked for bRecomputeTangent.
	return IsGPUSkinCacheAvailable(InPlatform) && (RenderSection.bRecomputeTangent || GPUSkinCacheStoreDuplicatedVertices());
}

static bool RequiresDuplicateVerticesInCook(const ITargetPlatform& InTargetPlatform, FSkelMeshRenderSection const& RenderSection)
{
	TArray<FName> TargetedShaderFormats;
	InTargetPlatform.GetAllTargetedShaderFormats(TargetedShaderFormats);
	for (int32 FormatIndex = 0; FormatIndex < TargetedShaderFormats.Num(); ++FormatIndex)
	{
		const EShaderPlatform LegacyShaderPlatform = ShaderFormatToLegacyShaderPlatform(TargetedShaderFormats[FormatIndex]);
		if (RequiresDuplicateVerticesInCook(LegacyShaderPlatform, RenderSection))
		{
			return true;
		}
	}
	return false;
}

static bool IsMeshDeformerSupportedByCookTargetPlatform(const ITargetPlatform& InTargetPlatform)
{
	TArray<FName> TargetedShaderFormats;
	InTargetPlatform.GetAllTargetedShaderFormats(TargetedShaderFormats);
	for (int32 FormatIndex = 0; FormatIndex < TargetedShaderFormats.Num(); ++FormatIndex)
	{
		const EShaderPlatform LegacyShaderPlatform = ShaderFormatToLegacyShaderPlatform(TargetedShaderFormats[FormatIndex]);
		
		if (AreMeshDeformersAvailable(LegacyShaderPlatform))
		{
			return true;
		}
		
	}
	return false;
}

// Serialization.
FArchive& operator<<(FArchive& Ar, FSkelMeshRenderSection& S)
{
	const uint8 DuplicatedVertices = 1;

	Ar.UsingCustomVersion(FUE5MainStreamObjectVersion::GUID);
	Ar.UsingCustomVersion(FRecomputeTangentCustomVersion::GUID);
	Ar.UsingCustomVersion(FUE5ReleaseStreamObjectVersion::GUID);

	// Strip DuplicatedVerticesBuffer from cook if it won't be needed.
	uint8 ClassDataStripFlags = 0;
	if (Ar.IsCooking() && !RequiresDuplicateVerticesInCook(*Ar.CookingTarget(), S))
	{
		ClassDataStripFlags |= DuplicatedVertices;
	}

	// When data is cooked for server platform some of the
	// variables are not serialized so that they're always
	// set to their initial values (for safety)
	FStripDataFlags StripFlags(Ar, ClassDataStripFlags);

	Ar << S.MaterialIndex;
	Ar << S.BaseIndex;
	Ar << S.NumTriangles;
	Ar << S.bRecomputeTangent;
	if (Ar.CustomVer(FRecomputeTangentCustomVersion::GUID) >= FRecomputeTangentCustomVersion::RecomputeTangentVertexColorMask)
	{
		Ar << S.RecomputeTangentsVertexMaskChannel;
	}
	else
	{
		// Our default is not to use vertex color as mask
		S.RecomputeTangentsVertexMaskChannel = ESkinVertexColorChannel::None;
	}
	Ar << S.bCastShadow;
	if (Ar.CustomVer(FUE5MainStreamObjectVersion::GUID) >= FUE5MainStreamObjectVersion::SkelMeshSectionVisibleInRayTracingFlagAdded)
	{
		Ar << S.bVisibleInRayTracing;
	}
	else
	{
		// default is to be visible in ray tracing - which is consistent with behaviour before adding this member
		S.bVisibleInRayTracing = true;
	}
	Ar << S.BaseVertexIndex;

	if (Ar.CustomVer(FUE5ReleaseStreamObjectVersion::GUID) < FUE5ReleaseStreamObjectVersion::AddClothMappingLODBias)
	{
		constexpr int32 ClothLODBias = 0;  // There isn't any cloth LOD bias prior to this version
		S.ClothMappingDataLODs.SetNum(1);
		Ar << S.ClothMappingDataLODs[ClothLODBias];
	}
	else
	{
		Ar << S.ClothMappingDataLODs;
	}

	Ar << S.BoneMap;
	Ar << S.NumVertices;

	if (Ar.CustomVer(FUE5MainStreamObjectVersion::GUID) >= FUE5MainStreamObjectVersion::SkeletalMeshUnifiedBoneMap)
	{
		const uint32 UnifiedBoneMapBit = 1u << 31;
		uint32 PackedBits = S.MaxBoneInfluences | (S.bUnifiedBoneMap ? UnifiedBoneMapBit : 0);
		Ar << PackedBits;
		S.bUnifiedBoneMap   = (PackedBits &  UnifiedBoneMapBit) != 0;
		S.MaxBoneInfluences = (PackedBits & ~UnifiedBoneMapBit);
	}
	else
	{
		Ar << S.MaxBoneInfluences;
		S.bUnifiedBoneMap = false;
	}

	Ar << S.CorrespondClothAssetIndex;
	Ar << S.ClothingData;
	if (!StripFlags.IsClassDataStripped(DuplicatedVertices))
	{
		Ar << S.DuplicatedVerticesBuffer;
	}
	Ar << S.bDisabled;

	return Ar;
}

void FSkeletalMeshLODRenderData::InitMorphResources()
{
	if (!MorphTargetVertexInfoBuffers.IsRHIInitialized() && MorphTargetVertexInfoBuffers.IsMorphCPUDataValid() && MorphTargetVertexInfoBuffers.NumTotalBatches > 0)
	{
		// The morph target could have been loaded prior but gets streamed in again, so we have to release the resources here, otherwise
		// FRenderResource::InitResource does nothing and leaves IsRHIInitialized() as false, which results in no morphs appearing. 
		if (MorphTargetVertexInfoBuffers.IsInitialized())
		{
			BeginReleaseResource(&MorphTargetVertexInfoBuffers, &UE::RenderCommandPipe::SkeletalMesh);
		}
		BeginInitResource(&MorphTargetVertexInfoBuffers, &UE::RenderCommandPipe::SkeletalMesh);
	}
}

void FSkeletalMeshLODRenderData::InitResources(bool bNeedsVertexColors, int32 LODIndex, USkinnedAsset* Owner)
{
	const FName OwnerName(USkinnedAsset::GetLODPathName(Owner, LODIndex));

	if (bStreamedDataInlined)
	{
		IncrementMemoryStats(bNeedsVertexColors);
	}

	MultiSizeIndexContainer.SetOwnerName(OwnerName);
	MultiSizeIndexContainer.InitResources();

	BeginInitResource(OwnerName, &StaticVertexBuffers.PositionVertexBuffer, &UE::RenderCommandPipe::SkeletalMesh);
	BeginInitResource(OwnerName, &StaticVertexBuffers.StaticMeshVertexBuffer, &UE::RenderCommandPipe::SkeletalMesh);

	SkinWeightVertexBuffer.SetOwnerName(OwnerName);
	SkinWeightVertexBuffer.BeginInitResources();

	if (bNeedsVertexColors)
	{
		// Only init the color buffer if the mesh has vertex colors
		BeginInitResource(OwnerName, &StaticVertexBuffers.ColorVertexBuffer, &UE::RenderCommandPipe::SkeletalMesh);
	}

	if (ClothVertexBuffer.GetNumVertices() > 0)
	{
		// Only init the clothing buffer if the mesh has clothing data
		BeginInitResource(OwnerName, &ClothVertexBuffer, &UE::RenderCommandPipe::SkeletalMesh);
	}

	// If we have duplicate vertex data then initialize them
	for (FSkelMeshRenderSection& RenderSection : RenderSections)
	{
		if(RenderSection.DuplicatedVerticesBuffer.DupVertData.Num())
		{
			// No need to discard CPU data in cooked builds as bNeedsCPUAccess is false (see FDuplicatedVerticesBuffer constructor), 
			// so it'd be auto-discarded after the RHI has copied the resource data. Keep CPU data when in the editor for geometry operations.
			BeginInitResource(OwnerName, &RenderSection.DuplicatedVerticesBuffer, &UE::RenderCommandPipe::SkeletalMesh);
		}
	}

	if (Owner)
	{
		const TArray<TObjectPtr<UMorphTarget>>& MorphTargets = Owner->GetMorphTargets();
		// Always make sure the morph target resources are in an initialized state. This should not get hit since the data should come with the load.
		bool bNeedsMorphTargetRenderData = !MorphTargets.IsEmpty() && !MorphTargetVertexInfoBuffers.IsMorphResourcesInitialized();
		if (bNeedsMorphTargetRenderData)
		{
			const FSkeletalMeshLODInfo* SkeletalMeshLODInfo = Owner->GetLODInfo(LODIndex);
			MorphTargetVertexInfoBuffers.InitMorphResources(GMaxRHIShaderPlatform, RenderSections, MorphTargets, StaticVertexBuffers.StaticMeshVertexBuffer.GetNumVertices(), LODIndex, SkeletalMeshLODInfo->MorphTargetPositionErrorTolerance);
		}
	}
	
	if (!MorphTargetVertexInfoBuffers.IsRHIInitialized() && MorphTargetVertexInfoBuffers.IsMorphCPUDataValid() && MorphTargetVertexInfoBuffers.NumTotalBatches > 0)
	{
		BeginInitResource(OwnerName, &MorphTargetVertexInfoBuffers, &UE::RenderCommandPipe::SkeletalMesh);
	}

	VertexAttributeBuffers.InitResources();
	
	BeginInitResource(OwnerName, &HalfEdgeBuffer, &UE::RenderCommandPipe::SkeletalMesh);

#if RHI_RAYTRACING
	if (IsRayTracingAllowed())
	{
		if (SourceRayTracingGeometry.RawDataHeader.IsValid())
		{
			uint32 TrianglesCount = 0;

			TArray<FRayTracingGeometrySegment> GeometrySections;
			GeometrySections.Reserve(RenderSections.Num());

			for (const FSkelMeshRenderSection& Section : RenderSections)
			{
				FRayTracingGeometrySegment Segment;
				Segment.FirstPrimitive = Section.BaseIndex / 3;
				Segment.NumPrimitives = Section.NumTriangles;
				Segment.bEnabled = !Section.bDisabled && Section.bVisibleInRayTracing;
				GeometrySections.Add(Segment);

				TrianglesCount += Section.NumTriangles;
			}

			FRayTracingGeometryInitializer Initializer;
			Initializer.DebugName = (Owner ? Owner->GetFName() : NAME_None);
			Initializer.OwnerName = OwnerName;
			Initializer.bTemplate = true;
			Initializer.TotalPrimitiveCount = TrianglesCount;
			Initializer.Segments = GeometrySections;

			SourceRayTracingGeometry.SetInitializer(Initializer);

			SourceRayTracingGeometry.LODIndex = LODIndex;

			BeginInitResource(OwnerName, &SourceRayTracingGeometry, &UE::RenderCommandPipe::SkeletalMesh);
		}
	}
#endif
}

void FSkeletalMeshLODRenderData::ReleaseResources()
{
	DecrementMemoryStats();

	MultiSizeIndexContainer.ReleaseResources();
	
	BeginReleaseResource(&StaticVertexBuffers.PositionVertexBuffer, &UE::RenderCommandPipe::SkeletalMesh);
	BeginReleaseResource(&StaticVertexBuffers.StaticMeshVertexBuffer, &UE::RenderCommandPipe::SkeletalMesh);
	SkinWeightVertexBuffer.BeginReleaseResources();
	BeginReleaseResource(&StaticVertexBuffers.ColorVertexBuffer, &UE::RenderCommandPipe::SkeletalMesh);
	BeginReleaseResource(&ClothVertexBuffer, &UE::RenderCommandPipe::SkeletalMesh);
	for (FSkelMeshRenderSection& RenderSection : RenderSections)
	{
		BeginReleaseResource(&RenderSection.DuplicatedVerticesBuffer, &UE::RenderCommandPipe::SkeletalMesh);
	}
	BeginReleaseResource(&MorphTargetVertexInfoBuffers, &UE::RenderCommandPipe::SkeletalMesh);

	DEC_DWORD_STAT_BY(STAT_SkeletalMeshVertexMemory, SkinWeightProfilesData.GetResourcesSize());
	SkinWeightProfilesData.ReleaseResources();

	VertexAttributeBuffers.ReleaseResources();
	
	BeginReleaseResource(&HalfEdgeBuffer, &UE::RenderCommandPipe::SkeletalMesh);
	
#if RHI_RAYTRACING
	if (IsRayTracingAllowed())
	{
		BeginReleaseResource(&SourceRayTracingGeometry, &UE::RenderCommandPipe::SkeletalMesh);
		BeginReleaseResource(&StaticRayTracingGeometry, &UE::RenderCommandPipe::SkeletalMesh);
	}
#endif
}

void FSkeletalMeshLODRenderData::IncrementMemoryStats(bool bNeedsVertexColors)
{
	INC_DWORD_STAT_BY(STAT_SkeletalMeshIndexMemory, MultiSizeIndexContainer.IsIndexBufferValid() ? (MultiSizeIndexContainer.GetIndexBuffer()->Num() * MultiSizeIndexContainer.GetDataTypeSize()) : 0);
	INC_DWORD_STAT_BY(STAT_SkeletalMeshVertexMemory, StaticVertexBuffers.PositionVertexBuffer.GetStride() * StaticVertexBuffers.PositionVertexBuffer.GetNumVertices());
	INC_DWORD_STAT_BY(STAT_SkeletalMeshVertexMemory, StaticVertexBuffers.StaticMeshVertexBuffer.GetResourceSize());
	INC_DWORD_STAT_BY(STAT_SkeletalMeshVertexMemory, SkinWeightVertexBuffer.GetVertexDataSize());

	if (bNeedsVertexColors)
	{
		INC_DWORD_STAT_BY(STAT_SkeletalMeshVertexMemory, StaticVertexBuffers.ColorVertexBuffer.GetAllocatedSize());
	}

	if (ClothVertexBuffer.GetNumVertices() > 0)
	{
		INC_DWORD_STAT_BY(STAT_SkeletalMeshVertexMemory, ClothVertexBuffer.GetVertexDataSize());
	}
}

void FSkeletalMeshLODRenderData::DecrementMemoryStats()
{
	DEC_DWORD_STAT_BY(STAT_SkeletalMeshIndexMemory, MultiSizeIndexContainer.IsIndexBufferValid() ? (MultiSizeIndexContainer.GetIndexBuffer()->Num() * MultiSizeIndexContainer.GetDataTypeSize()) : 0);
	DEC_DWORD_STAT_BY(STAT_SkeletalMeshVertexMemory, StaticVertexBuffers.PositionVertexBuffer.GetStride() * StaticVertexBuffers.PositionVertexBuffer.GetNumVertices());
	DEC_DWORD_STAT_BY(STAT_SkeletalMeshVertexMemory, StaticVertexBuffers.StaticMeshVertexBuffer.GetResourceSize());

	DEC_DWORD_STAT_BY(STAT_SkeletalMeshVertexMemory, SkinWeightVertexBuffer.GetVertexDataSize());
	DEC_DWORD_STAT_BY(STAT_SkeletalMeshVertexMemory, StaticVertexBuffers.ColorVertexBuffer.GetAllocatedSize());
	DEC_DWORD_STAT_BY(STAT_SkeletalMeshVertexMemory, ClothVertexBuffer.GetVertexDataSize());
}

#if WITH_EDITOR
void FSkeletalMeshLODRenderData::BuildFromLODModel(
	const FSkeletalMeshLODModel* InLODModel,
	TConstArrayView<FSkeletalMeshVertexAttributeInfo> InVertexAttributeInfos,
	const FBuildSettings& InBuildSettings
	)
{
	const ESkeletalMeshVertexFlags& BuildFlags = InBuildSettings.BuildFlags;

	const bool bUseFullPrecisionUVs = EnumHasAllFlags(BuildFlags, ESkeletalMeshVertexFlags::UseFullPrecisionUVs);
	const bool bUseHighPrecisionTangentBasis = EnumHasAllFlags(BuildFlags, ESkeletalMeshVertexFlags::UseHighPrecisionTangentBasis);
	const bool bHasVertexColors = EnumHasAllFlags(BuildFlags, ESkeletalMeshVertexFlags::HasVertexColors);
	const bool bUseBackwardsCompatibleF16TruncUVs = EnumHasAllFlags(BuildFlags, ESkeletalMeshVertexFlags::UseBackwardsCompatibleF16TruncUVs);
	const bool bUseHighPrecisionWeights = EnumHasAllFlags(BuildFlags, ESkeletalMeshVertexFlags::UseHighPrecisionWeights);

	const FSkeletalMeshUnifiedBoneIndices UnifiedBoneIndices(InBuildSettings.UnifiedBoneMap);

	// Copy required info from source sections
	RenderSections.Empty();
	for (int32 SectionIndex = 0; SectionIndex < InLODModel->Sections.Num(); SectionIndex++)
	{
		const FSkelMeshSection& ModelSection = InLODModel->Sections[SectionIndex];

		FSkelMeshRenderSection& NewRenderSection = RenderSections.Emplace_GetRef();
		NewRenderSection.MaterialIndex = ModelSection.MaterialIndex;
		NewRenderSection.BaseIndex = ModelSection.BaseIndex;
		NewRenderSection.NumTriangles = ModelSection.NumTriangles;
		NewRenderSection.bRecomputeTangent = ModelSection.bRecomputeTangent;
		NewRenderSection.RecomputeTangentsVertexMaskChannel = ModelSection.RecomputeTangentsVertexMaskChannel;
		NewRenderSection.bCastShadow = ModelSection.bCastShadow;
		NewRenderSection.bVisibleInRayTracing = ModelSection.bVisibleInRayTracing;
		NewRenderSection.BaseVertexIndex = ModelSection.BaseVertexIndex;
		NewRenderSection.ClothMappingDataLODs = ModelSection.ClothMappingDataLODs;
		NewRenderSection.BoneMap = ModelSection.BoneMap;
		NewRenderSection.NumVertices = ModelSection.NumVertices;
		NewRenderSection.MaxBoneInfluences = ModelSection.MaxBoneInfluences;
		NewRenderSection.CorrespondClothAssetIndex = ModelSection.CorrespondClothAssetIndex;
		NewRenderSection.ClothingData = ModelSection.ClothingData;
		NewRenderSection.DuplicatedVerticesBuffer.Init(ModelSection.NumVertices, ModelSection.OverlappingVertices);
		NewRenderSection.bDisabled = ModelSection.bDisabled;

		if (UnifiedBoneIndices.IsValid())
		{
			NewRenderSection.BoneMap = UnifiedBoneIndices.GetBoneMap();
			NewRenderSection.bUnifiedBoneMap = true;

			// Assign the bone map up the chain to the LOD.
			UnifiedBoneMap = NewRenderSection.BoneMap;
		}
		else
		{
			NewRenderSection.BoneMap = ModelSection.BoneMap;
			NewRenderSection.bUnifiedBoneMap = false;
		}
	}

	TArray<FSoftSkinVertex> Vertices;
	InLODModel->GetVertices(Vertices, &UnifiedBoneIndices);

	// match UV and tangent precision for mesh vertex buffer to setting from parent mesh
	StaticVertexBuffers.StaticMeshVertexBuffer.SetUseFullPrecisionUVs(bUseFullPrecisionUVs);
	StaticVertexBuffers.StaticMeshVertexBuffer.SetUseHighPrecisionTangentBasis(bUseHighPrecisionTangentBasis);

	// init vertex buffer with the vertex array
	StaticVertexBuffers.PositionVertexBuffer.Init(Vertices.Num());
	StaticVertexBuffers.StaticMeshVertexBuffer.Init(Vertices.Num(), InLODModel->NumTexCoords);

	for (int32 i = 0; i < Vertices.Num(); i++)
	{
		StaticVertexBuffers.PositionVertexBuffer.VertexPosition(i) = Vertices[i].Position;
		StaticVertexBuffers.StaticMeshVertexBuffer.SetVertexTangents(i, Vertices[i].TangentX, Vertices[i].TangentY, Vertices[i].TangentZ);
		for (uint32 j = 0; j < InLODModel->NumTexCoords; j++)
		{
			StaticVertexBuffers.StaticMeshVertexBuffer.SetVertexUV(i, j, Vertices[i].UVs[j], bUseBackwardsCompatibleF16TruncUVs);
		}
	}

	if (!bUseHighPrecisionWeights)
	{
		// Re-normalize the weights for 8-bits to ensure that they are distributed using the old algorithm
		// from MeshUtilities.cpp
		uint8 InfluenceWeights[MAX_TOTAL_INFLUENCES] = {0};

		for (FSoftSkinVertex& Vertex: Vertices)
		{
			uint32	TotalInfluenceWeight = 0;
			int32	MaxInfluenceIndex = 0;
			for (; MaxInfluenceIndex < MAX_TOTAL_INFLUENCES && Vertex.InfluenceWeights[MaxInfluenceIndex]; MaxInfluenceIndex++)
			{
				InfluenceWeights[MaxInfluenceIndex] = static_cast<uint8>(Vertex.InfluenceWeights[MaxInfluenceIndex] >> 8);
				if (InfluenceWeights[MaxInfluenceIndex] == 0)
				{
					break;
				}
				TotalInfluenceWeight += InfluenceWeights[MaxInfluenceIndex];
			}
			InfluenceWeights[0] += 255 - TotalInfluenceWeight;

			FMemory::Memzero(Vertex.InfluenceWeights);
			for (int32 Index = 0; Index < MaxInfluenceIndex; Index++)
			{
				// Map from 8-bit range to 16-bit range.
				Vertex.InfluenceWeights[Index] = (static_cast<uint16>(InfluenceWeights[Index]) << 8) | InfluenceWeights[Index];
			}
		}
	}

	// Init skin weight buffer
	SkinWeightVertexBuffer.SetNeedsCPUAccess(true);
	SkinWeightVertexBuffer.SetMaxBoneInfluences(InLODModel->GetMaxBoneInfluences());

	if (UnifiedBoneIndices.IsValid())
	{
		SkinWeightVertexBuffer.SetUse16BitBoneIndex(UnifiedBoneIndices.GetNumUnifiedBones() > 256);
	}
	else
	{
		SkinWeightVertexBuffer.SetUse16BitBoneIndex(InLODModel->DoSectionsUse16BitBoneIndex());
	}

	SkinWeightVertexBuffer.SetUse16BitBoneWeight(bUseHighPrecisionWeights);
	SkinWeightVertexBuffer.Init(Vertices);

	// Init the color buffer if this mesh has vertex colors.
	if (bHasVertexColors && Vertices.Num() > 0 && StaticVertexBuffers.ColorVertexBuffer.GetAllocatedSize() == 0)
	{
		StaticVertexBuffers.ColorVertexBuffer.InitFromColorArray(&Vertices[0].Color, Vertices.Num(), sizeof(FSoftSkinVertex));
	}

	if (InLODModel->HasClothData())
	{
		TArray<FMeshToMeshVertData> MappingData;
		TArray<FClothBufferIndexMapping> ClothIndexMapping;
		InLODModel->GetClothMappingData(MappingData, ClothIndexMapping);
		ClothVertexBuffer.Init(MappingData, ClothIndexMapping);
	}

	const uint8 DataTypeSize = (InLODModel->NumVertices < MAX_uint16) ? sizeof(uint16) : sizeof(uint32);

	MultiSizeIndexContainer.RebuildIndexBuffer(DataTypeSize, InLODModel->IndexBuffer);
	
	IMeshUtilities& MeshUtilities = FModuleManager::Get().LoadModuleChecked<IMeshUtilities>("MeshUtilities");

	// MorphTargetVertexInfoBuffers are created in InitResources

	SkinWeightProfilesData.Init(&SkinWeightVertexBuffer);
	// Generate runtime version of skin weight profile data, containing all required per-skin weight override data
	for (const auto& Pair : InLODModel->SkinWeightProfiles)
	{
		FRuntimeSkinWeightProfileData& Override = SkinWeightProfilesData.AddOverrideData(Pair.Key);
		MeshUtilities.GenerateRuntimeSkinWeightData(InLODModel, Pair.Value.SkinWeights, bUseHighPrecisionWeights, Override, &UnifiedBoneIndices);
	}

	for (const FSkeletalMeshVertexAttributeInfo& VertexAttributeInfo: InVertexAttributeInfos)
	{
		if (!VertexAttributeInfo.IsEnabledForRender())
		{
			continue;
		}

		const FSkeletalMeshModelVertexAttribute* VertexAttribute = InLODModel->VertexAttributes.Find(VertexAttributeInfo.Name);
		if (ensure(VertexAttribute))
		{
			VertexAttributeBuffers.AddAttribute(
				VertexAttributeInfo.Name,
				VertexAttributeInfo.DataType,
				InLODModel->NumVertices,
				VertexAttribute->ComponentCount,
				VertexAttribute->Values);
		}
	}

	if (InBuildSettings.bBuildHalfEdgeBuffers)
	{
		HalfEdgeBuffer.Init(*this);
	}
	
	ActiveBoneIndices = InLODModel->ActiveBoneIndices;
	RequiredBones = InLODModel->RequiredBones;

	if (UnifiedBoneIndices.IsValid())
	{
		// With unified bone maps, all sections share the cross-LOD unified BoneMap. This can
		// introduce bones into this LOD's Section.BoneMap that aren't in RequiredBones (they
		// came from other LODs or from alternate skin weight profiles, or from Nanite-encoded
		// vertex references that survive reduction). The invariant Section.BoneMap ⊆ RequiredBones
		// must hold for ASTP to write transforms for every vertex-referenced bone, otherwise
		// those slots stay uninitialized and vertices reading them render with garbage transforms.
		//
		// Union any unified BoneMap entries that aren't already in RequiredBones, then re-sort.
		bool bRequiredBonesChanged = false;

		for (const FSkelMeshRenderSection& Section : RenderSections)
		{
			for (FBoneIndexType BoneIndex : Section.BoneMap)
			{
				if (!RequiredBones.Contains(BoneIndex))
				{
					RequiredBones.Add(BoneIndex);
					ActiveBoneIndices.AddUnique(BoneIndex);
					bRequiredBonesChanged = true;
				}
			}
		}

		if (bRequiredBonesChanged)
		{
			RequiredBones.Sort();
			ActiveBoneIndices.Sort();
		}
	}

	// When a unified bone map is active, every section must share the exact same BoneMap.
	// That's the defining invariant of unified bone maps — all sections index into one
	// shared set of bone transforms. If section BoneMaps diverge in size or contents,
	// vertex bone indices reference different bones depending on which section they're in,
	// defeating the purpose of unified maps and causing rendering artifacts.
	if (UnifiedBoneIndices.IsValid() && RenderSections.Num() > 0)
	{
		const TArray<FBoneIndexType>& ReferenceBoneMap = RenderSections[0].BoneMap;
		for (int32 SectionIndex = 1; SectionIndex < RenderSections.Num(); ++SectionIndex)
		{
			const TArray<FBoneIndexType>& SectionBoneMap = RenderSections[SectionIndex].BoneMap;
			if (SectionBoneMap.Num() != ReferenceBoneMap.Num())
			{
				UE_LOG(LogSkeletalMesh, Error,
					TEXT("Unified BoneMap invariant violated: section %d has BoneMap size %d but section 0 has size %d"),
					SectionIndex, SectionBoneMap.Num(), ReferenceBoneMap.Num());
				continue;
			}

			for (int32 BoneMapIndex = 0; BoneMapIndex < SectionBoneMap.Num(); ++BoneMapIndex)
			{
				if (SectionBoneMap[BoneMapIndex] != ReferenceBoneMap[BoneMapIndex])
				{
					UE_LOG(LogSkeletalMesh, Error,
						TEXT("Unified BoneMap invariant violated: section %d BoneMap[%d] = %u, but section 0 BoneMap[%d] = %u"),
						SectionIndex, BoneMapIndex, SectionBoneMap[BoneMapIndex],
						BoneMapIndex, ReferenceBoneMap[BoneMapIndex]);
					break; // One error per section is enough.
				}
			}
		}
	}
}

#endif // WITH_EDITOR

void FSkeletalMeshLODRenderData::ReleaseCPUResources(bool bForStreaming)
{
	if (!GIsEditor && !IsRunningCommandlet())
	{
		if (MultiSizeIndexContainer.IsIndexBufferValid())
		{
			MultiSizeIndexContainer.GetIndexBuffer()->Empty();
		}

		SkinWeightVertexBuffer.CleanUp();
		StaticVertexBuffers.PositionVertexBuffer.CleanUp();
		StaticVertexBuffers.StaticMeshVertexBuffer.CleanUp();

		if (bForStreaming)
		{
			ClothVertexBuffer.CleanUp();
			StaticVertexBuffers.ColorVertexBuffer.CleanUp();
			SkinWeightProfilesData.ReleaseCPUResources();
			VertexAttributeBuffers.CleanUp();
			HalfEdgeBuffer.CleanUp();
		}
	}
}


void FSkeletalMeshLODRenderData::GetResourceSizeEx(FResourceSizeEx& CumulativeResourceSize) const
{
	if (MultiSizeIndexContainer.IsIndexBufferValid())
	{
		const FRawStaticIndexBuffer16or32Interface* IndexBuffer = MultiSizeIndexContainer.GetIndexBuffer();
		if (IndexBuffer)
		{
			CumulativeResourceSize.AddUnknownMemoryBytes(TEXT("IndexBuffer"), IndexBuffer->GetResourceDataSize());
		}
	}

	CumulativeResourceSize.AddUnknownMemoryBytes(TEXT("PositionVertexBuffer"), StaticVertexBuffers.PositionVertexBuffer.GetNumVertices() * StaticVertexBuffers.PositionVertexBuffer.GetStride());
	CumulativeResourceSize.AddUnknownMemoryBytes(TEXT("TexcoordBuffer and TangentBuffer"), StaticVertexBuffers.StaticMeshVertexBuffer.GetResourceSize());
	CumulativeResourceSize.AddUnknownMemoryBytes(TEXT("SkinWeightVertexBuffer"), SkinWeightVertexBuffer.GetVertexDataSize());
	CumulativeResourceSize.AddUnknownMemoryBytes(TEXT("ColorVertexBuffer"), StaticVertexBuffers.ColorVertexBuffer.GetAllocatedSize());
	CumulativeResourceSize.AddUnknownMemoryBytes(TEXT("ClothVertexBuffer"), ClothVertexBuffer.GetVertexDataSize());
	CumulativeResourceSize.AddUnknownMemoryBytes(TEXT("SkinWeightProfilesData"), SkinWeightProfilesData.GetResourcesSize());
	CumulativeResourceSize.AddUnknownMemoryBytes(TEXT("VertexAttributeData"), VertexAttributeBuffers.GetResourceSize());
	CumulativeResourceSize.AddUnknownMemoryBytes(TEXT("HalfEdgeBuffer"), HalfEdgeBuffer.GetResourceSize());
	
}

SIZE_T FSkeletalMeshLODRenderData::GetCPUAccessMemoryOverhead() const
{
	SIZE_T Result = 0;

	if (MultiSizeIndexContainer.IsIndexBufferValid())
	{
		const FRawStaticIndexBuffer16or32Interface* IndexBuffer = MultiSizeIndexContainer.GetIndexBuffer();
		Result += IndexBuffer && IndexBuffer->GetNeedsCPUAccess() ? IndexBuffer->GetResourceDataSize() : 0;
	}

	Result += StaticVertexBuffers.StaticMeshVertexBuffer.GetAllowCPUAccess() ? StaticVertexBuffers.StaticMeshVertexBuffer.GetResourceSize() : 0;
	Result += StaticVertexBuffers.PositionVertexBuffer.GetAllowCPUAccess() ? StaticVertexBuffers.PositionVertexBuffer.GetNumVertices() * StaticVertexBuffers.PositionVertexBuffer.GetStride() : 0;
	Result += StaticVertexBuffers.ColorVertexBuffer.GetAllowCPUAccess() ? StaticVertexBuffers.ColorVertexBuffer.GetAllocatedSize() : 0;
	Result += SkinWeightVertexBuffer.GetNeedsCPUAccess() ? SkinWeightVertexBuffer.GetVertexDataSize() : 0;
	Result += ClothVertexBuffer.GetVertexDataSize();
	Result += SkinWeightProfilesData.GetCPUAccessMemoryOverhead();
	return Result;
}

uint8 FSkeletalMeshLODRenderData::GenerateClassStripFlags(FArchive& Ar, const USkinnedAsset* OwnerMesh, int32 LODIdx)
{
#if WITH_EDITOR
	const bool bIsCook = Ar.IsCooking();
	const ITargetPlatform* CookTarget = Ar.CookingTarget();

	int32 MinMeshLod = 0;
	bool bMeshDisablesMinLodStrip = false;
	if (bIsCook)
	{
		MinMeshLod = OwnerMesh ? OwnerMesh->GetPlatformMinLODIdx(Ar.CookingTarget()) : 0;
		bMeshDisablesMinLodStrip = OwnerMesh ? OwnerMesh->GetDisableBelowMinLodStripping().GetValueForPlatform(CookTarget->GetPlatformInfo().IniPlatformName) : false;
	}
	const bool bWantToStripBelowMinLod = bIsCook && GStripSkeletalMeshLodsDuringCooking != 0 && MinMeshLod > LODIdx && !bMeshDisablesMinLodStrip;

	uint8 ClassDataStripFlags = 0;
	ClassDataStripFlags |= bWantToStripBelowMinLod ? CDSF_MinLodData : 0;
	return ClassDataStripFlags;
#else
	return 0;
#endif
}

bool FSkeletalMeshLODRenderData::IsLODCookedOut(const ITargetPlatform* TargetPlatform, const USkinnedAsset* SkinnedAsset, bool bIsBelowMinLOD)
{
	check(SkinnedAsset);
#if WITH_EDITOR
	if (!bIsBelowMinLOD)
	{
		return false;
	}

	if (!TargetPlatform)
	{
		TargetPlatform = GetTargetPlatformManagerRef().GetRunningTargetPlatform();
	}
	check(TargetPlatform);

	return !SkinnedAsset->GetEnableLODStreaming(TargetPlatform);
#else
	return false;
#endif
}

bool FSkeletalMeshLODRenderData::IsLODInlined(const ITargetPlatform* TargetPlatform, const USkinnedAsset* SkinnedAsset, int32 LODIdx, bool bIsBelowMinLOD)
{
	check(SkinnedAsset);
#if WITH_EDITOR
	if (!TargetPlatform)
	{
		TargetPlatform = GetTargetPlatformManagerRef().GetRunningTargetPlatform();
	}
	check(TargetPlatform);
	
	if (!SkinnedAsset->GetEnableLODStreaming(TargetPlatform))
	{
		return true;
	}

	if (bIsBelowMinLOD)
	{
		return false;
	}

	const int32 MaxNumStreamedLODs = SkinnedAsset->GetMaxNumStreamedLODs(TargetPlatform);
	const int32 NumLODs = SkinnedAsset->GetLODNum();
	const int32 NumStreamedLODs = FMath::Min(MaxNumStreamedLODs, NumLODs - 1);
	const int32 InlinedLODStartIdx = NumStreamedLODs;
	return LODIdx >= InlinedLODStartIdx;
#else
	return false;
#endif
}

int32 FSkeletalMeshLODRenderData::GetNumOptionalLODsAllowed(const ITargetPlatform* TargetPlatform, const USkinnedAsset* SkinnedAsset)
{
#if WITH_EDITOR
	check(TargetPlatform && SkinnedAsset);
	return SkinnedAsset->GetMaxNumOptionalLODs(TargetPlatform);
#else
	return 0;
#endif
}

bool FSkeletalMeshLODRenderData::ShouldForceKeepCPUResources()
{
#if !WITH_EDITOR
	static const auto CVar = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.FreeSkeletalMeshBuffers"));
	if (CVar)
	{
		return !CVar->GetValueOnAnyThread();
	}
#endif
	return true;
}

bool FSkeletalMeshLODRenderData::ShouldKeepCPUResources(const USkinnedAsset* SkinnedAsset, int32 LODIdx, bool bForceKeep)
{
	return bForceKeep
		|| SkinnedAsset->NeedCPUData(LODIdx);
}

class FSkeletalMeshLODSizeCounter : public FArchive
{
public:
	FSkeletalMeshLODSizeCounter()
		: Size(0)
	{
		SetIsSaving(true);
		SetIsPersistent(true);
		ArIsCountingMemory = true;
	}

	virtual void Serialize(void*, int64 Length) final override
	{
		Size += Length;
	}

	virtual int64 TotalSize() final override
	{
		return Size;
	}

private:
	int64 Size;
};

namespace SkeletalMeshResourceNames
{
	static const FName MultiSizeIndexContainerName("FSkeletalMeshLODRenderData.MultiSizeIndexContainer");
	static const FName StaticPositionVertexBufferName("FSkeletalMeshLODRenderData.StaticVertexBuffers.PositionVertexBuffer");
	static const FName StaticMeshVertexBufferName("FSkeletalMeshLODRenderData.StaticVertexBuffers.StaticMeshVertexBuffer");
	static const FName StaticColorVertexBufferName("FSkeletalMeshLODRenderData.StaticVertexBuffers.ColorVertexBuffer");
	static const FName ClothVertexBufferName("FSkeletalMeshLODRenderData.ClothVertexBuffer");
	static const FName SourceRayTracingGeometryName("FSkeletalMeshLODRenderData.SourceRayTracingGeometry");
	static const FName MorphTargetVertexInfoBuffersName("FSkeletalMeshLODRenderData.MorphTargetVertexInfoBuffers");
	static const FName HalfEdgeBufferName("FSkeletalMeshLODRenderData.HalfEdgeBuffer");
} // SkeletalMeshResourceNames

void FSkeletalMeshLODRenderData::SerializeStreamedData(FArchive& Ar, USkinnedAsset* Owner, int32 LODIdx, uint8 ClassDataStripFlags, bool bNeedsCPUAccess, bool bForceKeepCPUResources)
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	UE::TScopeLock Lock(RenderDataMutex);
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	
	Ar.UsingCustomVersion(FUE5SpecialProjectStreamObjectVersion::GUID);
	Ar.UsingCustomVersion(FUE5ReleaseStreamObjectVersion::GUID);
	Ar.UsingCustomVersion(FFortniteMainBranchObjectVersion::GUID);
	
	FStripDataFlags StripFlags(Ar, ClassDataStripFlags);

	// TODO: A lot of data in a render section is needed during initialization but maybe some can still be streamed
	//Ar << RenderSections;

	MultiSizeIndexContainer.Serialize(Ar, bNeedsCPUAccess);
	MultiSizeIndexContainer.GetIndexBuffer()->SetResourceName(SkeletalMeshResourceNames::MultiSizeIndexContainerName);

	if (Ar.IsLoading())
	{
		SkinWeightVertexBuffer.SetNeedsCPUAccess(bNeedsCPUAccess);
	}

	StaticVertexBuffers.PositionVertexBuffer.Serialize(Ar, bNeedsCPUAccess);
	StaticVertexBuffers.PositionVertexBuffer.SetResourceName(SkeletalMeshResourceNames::StaticPositionVertexBufferName);
	StaticVertexBuffers.StaticMeshVertexBuffer.Serialize(Ar, bNeedsCPUAccess);
	StaticVertexBuffers.StaticMeshVertexBuffer.SetResourceName(SkeletalMeshResourceNames::StaticMeshVertexBufferName);
	Ar << SkinWeightVertexBuffer;

	if (Owner && Owner->GetHasVertexColors())
	{
		StaticVertexBuffers.ColorVertexBuffer.Serialize(Ar, bNeedsCPUAccess);
		StaticVertexBuffers.ColorVertexBuffer.SetResourceName(SkeletalMeshResourceNames::StaticColorVertexBufferName);
	}

	if (Ar.IsLoading() && Ar.CustomVer(FUE5ReleaseStreamObjectVersion::GUID) < FUE5ReleaseStreamObjectVersion::RemovingTessellation && !StripFlags.IsClassDataStripped(CDSF_AdjacencyData_DEPRECATED))
	{
		FMultiSizeIndexContainer AdjacencyMultiSizeIndexContainer;
		AdjacencyMultiSizeIndexContainer.Serialize(Ar, bForceKeepCPUResources);
	}

	if (HasClothData())
	{
		Ar << ClothVertexBuffer;
		ClothVertexBuffer.SetResourceName(SkeletalMeshResourceNames::ClothVertexBufferName);
	}

	Ar << SkinWeightProfilesData;
	SkinWeightProfilesData.Init(&SkinWeightVertexBuffer);

	if (Ar.IsLoading() && !Ar.IsError() && Owner)
	{
		Owner->SetSkinWeightProfilesData(LODIdx, SkinWeightProfilesData);
	}

	Ar << SourceRayTracingGeometry.RawDataHeader;
	SourceRayTracingGeometry.RawData.BulkSerialize(Ar);
	SourceRayTracingGeometry.SetResourceName(SkeletalMeshResourceNames::SourceRayTracingGeometryName);

	if (Ar.IsLoading() && RayTracing::ShouldForceRuntimeBLAS())
	{
		SourceRayTracingGeometry.RawDataHeader = FRayTracingGeometryOfflineDataHeader();
		SourceRayTracingGeometry.RawData.Discard();
	}

	// Determine if morph target data should be cooked out for this platform
	EShaderPlatform MorphTargetShaderPlatform = GMaxRHIShaderPlatform;
	bool bSerializeCompressedMorphTargets = Ar.IsSaving() && FMorphTargetVertexInfoBuffers::IsPlatformShaderSupported(MorphTargetShaderPlatform) && Owner && Owner->GetMorphTargets().Num() > 0;
	if (Ar.IsCooking())
	{
		const ITargetPlatform* Platform = Ar.CookingTarget();
		
		bSerializeCompressedMorphTargets = false;

		// Make sure to avoid cooking the morph target data when build a server only executable
		if (!Platform->IsServerOnly() && Owner && Owner->GetMorphTargets().Num() > 0)
		{
			// Test if any of the supported shader formats supports SM5
			TArray<FName> ShaderFormats;
			Ar.CookingTarget()->GetAllTargetedShaderFormats(ShaderFormats);
			for (FName ShaderFormat : ShaderFormats)
			{
				const EShaderPlatform ShaderPlatform = ShaderFormatToLegacyShaderPlatform(ShaderFormat);
				if (FMorphTargetVertexInfoBuffers::IsPlatformShaderSupported(ShaderPlatform))
				{
					MorphTargetShaderPlatform = ShaderPlatform;
					bSerializeCompressedMorphTargets = true;
					break;
				}
			}
		}
	}

	if(Ar.CustomVer(FUE5SpecialProjectStreamObjectVersion::GUID) >= FUE5SpecialProjectStreamObjectVersion::SerializeSkeletalMeshMorphTargetRenderData)
	{
		Ar << bSerializeCompressedMorphTargets;

		if (bSerializeCompressedMorphTargets && Owner)
		{
#if WITH_EDITOR
			if (Ar.IsSaving())
			{
				FMorphTargetVertexInfoBuffers LocalMorphTargetVertexInfoBuffers;
				FMorphTargetVertexInfoBuffers* TargetMorphTargetVertexInfoBuffers;
				const FSkeletalMeshLODInfo* SkeletalMeshLODInfo = Owner->GetLODInfo(LODIdx);
				const TArray<UMorphTarget*>& MorphTargets = Owner->GetMorphTargets();

				// The CPU data could have already been destroyed by this point, which happens when the RHI is initialized.  If possible, use the MorphTargetVertexInfoBuffers.
				if (!MorphTargetVertexInfoBuffers.IsMorphResourcesInitialized())
				{
					TargetMorphTargetVertexInfoBuffers = &MorphTargetVertexInfoBuffers;
					TargetMorphTargetVertexInfoBuffers->InitMorphResources(MorphTargetShaderPlatform, RenderSections, MorphTargets, StaticVertexBuffers.StaticMeshVertexBuffer.GetNumVertices(), LODIdx, SkeletalMeshLODInfo->MorphTargetPositionErrorTolerance);
				}
				else if (MorphTargetVertexInfoBuffers.IsMorphCPUDataValid() && !MorphTargetVertexInfoBuffers.IsRHIInitialized())
				{
					TargetMorphTargetVertexInfoBuffers = &MorphTargetVertexInfoBuffers;
					check(TargetMorphTargetVertexInfoBuffers->IsMorphCPUDataValid());
				}
				else
				{
					// Fallback for when the RHI data has already been created.  Create a local version of the morph data, and serialize that
					TargetMorphTargetVertexInfoBuffers = &LocalMorphTargetVertexInfoBuffers;
					TargetMorphTargetVertexInfoBuffers->InitMorphResources(MorphTargetShaderPlatform, RenderSections, MorphTargets, StaticVertexBuffers.StaticMeshVertexBuffer.GetNumVertices(), LODIdx, SkeletalMeshLODInfo->MorphTargetPositionErrorTolerance);
					check(TargetMorphTargetVertexInfoBuffers->IsMorphCPUDataValid());
				}

				Ar << *TargetMorphTargetVertexInfoBuffers;
			}
			else
#endif
			{
				Ar << MorphTargetVertexInfoBuffers;
				MorphTargetVertexInfoBuffers.SetResourceName(SkeletalMeshResourceNames::MorphTargetVertexInfoBuffersName);

#if DO_CHECK
				const FSkeletalMeshLODInfo* SkeletalMeshLODInfo = Owner->GetLODInfo(LODIdx);
				const float ExpectedPositionPrecision = UE::MorphTargetVertexCodec::ComputePositionPrecision(SkeletalMeshLODInfo->MorphTargetPositionErrorTolerance);
				checkf(FMath::IsNearlyEqualByULP(ExpectedPositionPrecision, MorphTargetVertexInfoBuffers.GetPositionPrecision()), TEXT("Morph target render data position tollerance in the LOD info (%f) for the skeleton %s does not match the expected tolerance (%f)."),
					MorphTargetVertexInfoBuffers.GetPositionPrecision(), *Owner->GetName(), ExpectedPositionPrecision);
#endif
			}
		}
	}
	if (Ar.CustomVer(FUE5MainStreamObjectVersion::GUID) >= FUE5MainStreamObjectVersion::SkeletalVertexAttributes)
	{
		Ar << VertexAttributeBuffers;
	}
	
	if (Ar.CustomVer(FFortniteMainBranchObjectVersion::GUID) >= FFortniteMainBranchObjectVersion::SkeletalHalfEdgeData)
	{
		uint8 ClassStripFlag = 0;
		const uint8 MeshDeformerStripFlag = 1;
		if (Ar.IsCooking() && !IsMeshDeformerSupportedByCookTargetPlatform(*Ar.CookingTarget()))
		{
			ClassStripFlag |= MeshDeformerStripFlag;
		}
		
		FStripDataFlags MeshDeformerStripFlags(Ar, ClassStripFlag);
		
		if (!MeshDeformerStripFlags.IsClassDataStripped(MeshDeformerStripFlag))
		{
			Ar << HalfEdgeBuffer;
			HalfEdgeBuffer.SetResourceName(SkeletalMeshResourceNames::HalfEdgeBufferName);
		}
	}
	
}

void FSkeletalMeshLODRenderData::SerializeAvailabilityInfo(FArchive& Ar, int32 LODIdx, bool bAdjacencyDataStripped, bool bNeedsCPUAccess)
{
	Ar.UsingCustomVersion(FUE5ReleaseStreamObjectVersion::GUID);
	MultiSizeIndexContainer.SerializeMetaData(Ar, bNeedsCPUAccess);
	if (Ar.IsLoading() && Ar.CustomVer(FUE5ReleaseStreamObjectVersion::GUID) < FUE5ReleaseStreamObjectVersion::RemovingTessellation && !bAdjacencyDataStripped)
	{
		FMultiSizeIndexContainer AdjacencyMultiSizeIndexContainer;
		AdjacencyMultiSizeIndexContainer.SerializeMetaData(Ar, bNeedsCPUAccess);
	}
	StaticVertexBuffers.StaticMeshVertexBuffer.SerializeMetaData(Ar);
	StaticVertexBuffers.PositionVertexBuffer.SerializeMetaData(Ar);
	StaticVertexBuffers.ColorVertexBuffer.SerializeMetaData(Ar);
	if (Ar.IsLoading())
	{
		SkinWeightVertexBuffer.SetNeedsCPUAccess(bNeedsCPUAccess);
	}
	SkinWeightVertexBuffer.SerializeMetaData(Ar);
	if (HasClothData())
	{
		ClothVertexBuffer.SerializeMetaData(Ar);
	}
	SkinWeightProfilesData.SerializeMetaData(Ar);
	SkinWeightProfilesData.Init(&SkinWeightVertexBuffer);

	{
		Ar << SourceRayTracingGeometry.RawDataHeader;

		if (Ar.IsLoading() && RayTracing::ShouldForceRuntimeBLAS())
		{
			SourceRayTracingGeometry.RawDataHeader = FRayTracingGeometryOfflineDataHeader();
		}
	}
}

void FSkeletalMeshLODRenderData::Serialize(FArchive& Ar, UObject* Owner, int32 Idx)
{
	DECLARE_SCOPE_CYCLE_COUNTER(TEXT("FSkeletalMeshLODRenderData::Serialize"), STAT_SkeletalMeshLODRenderData_Serialize, STATGROUP_LoadTime);

	// If we are loading in data, ensure the flag is set to false.
	if (Ar.IsLoading())
	{
		SetIsDataReady(false);
	}

	USkinnedAsset* OwnerMesh = CastChecked<USkinnedAsset>(Owner);
	
	// Shouldn't needed but to make some static analyzers happy
	if (!OwnerMesh)
	{
		return;
	}

	bool bUsingCookedEditorData = false;
#if WITH_EDITORONLY_DATA
	bUsingCookedEditorData = Owner->GetOutermost()->bIsCookedForEditor;
#endif
	
	// Actual flags used during serialization
	const uint8 ClassDataStripFlags = GenerateClassStripFlags(Ar, OwnerMesh, Idx);
	FStripDataFlags StripFlags(Ar, ClassDataStripFlags);

#if WITH_EDITOR
	const bool bIsBelowMinLOD = StripFlags.IsClassDataStripped(CDSF_MinLodData)
		|| (Ar.IsCooking() && OwnerMesh && Idx < OwnerMesh->GetPlatformMinLODIdx(Ar.CookingTarget()));
#else
	const bool bIsBelowMinLOD = false;
#endif
	bool bIsLODCookedOut = false;
	bool bInlined = false;

	if (Ar.IsSaving() && !Ar.IsCooking() && !!(Ar.GetPortFlags() & PPF_Duplicate))
	{
		bInlined = bStreamedDataInlined;
		bIsLODCookedOut = bIsBelowMinLOD && bInlined;
		Ar << bIsLODCookedOut;
		Ar << bInlined;
	}
	else
	{
		bIsLODCookedOut = IsLODCookedOut(Ar.CookingTarget(), OwnerMesh, bIsBelowMinLOD);
		Ar << bIsLODCookedOut;

		bInlined = bIsLODCookedOut || IsLODInlined(Ar.CookingTarget(), OwnerMesh, Idx, bIsBelowMinLOD);
		Ar << bInlined;
		bStreamedDataInlined = bInlined;
	}

	// Skeletal mesh buffers are kept in CPU memory after initialization to support merging of skeletal meshes.
	const bool bForceKeepCPUResources = ShouldForceKeepCPUResources();
	bool bNeedsCPUAccess = bForceKeepCPUResources;

	if (!StripFlags.IsAudioVisualDataStripped())
	{
		// set cpu skinning flag on the vertex buffer so that the resource arrays know if they need to be CPU accessible
		bNeedsCPUAccess = ShouldKeepCPUResources(OwnerMesh, Idx, bForceKeepCPUResources);
	}

	if (Ar.IsFilterEditorOnly())
	{
		if (bNeedsCPUAccess)
		{
			UE_LOGF(LogSkeletalMesh, Verbose, "[%ls] Skeletal Mesh is marked for CPU read.", *OwnerMesh->GetName());
		}
	}

	Ar << RequiredBones;

	if (!StripFlags.IsAudioVisualDataStripped() && !bIsLODCookedOut)
	{
		Ar << RenderSections;
		Ar << ActiveBoneIndices;

		UnifiedBoneMap = !RenderSections.IsEmpty() && RenderSections[0].bUnifiedBoneMap ? RenderSections[0].BoneMap : TConstArrayView<FBoneIndexType>();

#if WITH_EDITOR
		if (Ar.IsSaving())
		{
			FSkeletalMeshLODSizeCounter LODSizeCounter;
			LODSizeCounter.SetSavePackageData(Ar.GetSavePackageData());
			LODSizeCounter.SetByteSwapping(Ar.IsByteSwapping());
			SerializeStreamedData(LODSizeCounter, OwnerMesh, Idx, ClassDataStripFlags, bNeedsCPUAccess, bForceKeepCPUResources);
			BuffersSize = LODSizeCounter.TotalSize();
		}
#endif
		Ar << BuffersSize;

		if (Ar.IsFilterEditorOnly())
		{
			bool bDiscardBulkData = false;

#if WITH_EDITOR
			if (Ar.IsSaving())
			{
				const int32 MaxNumOptionalLODs = GetNumOptionalLODsAllowed(Ar.CookingTarget(), OwnerMesh);
				const int32 OptionalLODIdx = OwnerMesh->GetPlatformMinLODIdx(Ar.CookingTarget()) - Idx;
				bDiscardBulkData = OptionalLODIdx > MaxNumOptionalLODs;

				TArray<uint8> TmpBuff;
				if (!bDiscardBulkData)
				{
					FMemoryWriter MemWriter(TmpBuff, true);
					MemWriter.SetSavePackageData(Ar.GetSavePackageData());
					MemWriter.SetByteSwapping(Ar.IsByteSwapping());
					SerializeStreamedData(MemWriter, OwnerMesh, Idx, ClassDataStripFlags, bNeedsCPUAccess, bForceKeepCPUResources);
				}

				const uint32 OldBulkDataFlags = StreamingBulkData.GetBulkDataFlags();
				StreamingBulkData.ClearBulkDataFlags(0xffffffffu);
				if (TmpBuff.Num() > 0)
				{
					StreamingBulkData.Lock(LOCK_READ_WRITE);
					void* BulkDataMem = StreamingBulkData.Realloc(TmpBuff.Num());
					FMemory::Memcpy(BulkDataMem, TmpBuff.GetData(), TmpBuff.Num());
					StreamingBulkData.Unlock();
				}

				bIsLODOptional = bIsBelowMinLOD;

				const uint32 BulkDataInlineFlags = bInlined ? BULKDATA_ForceInlinePayload | BULKDATA_SingleUse : BULKDATA_Force_NOT_InlinePayload;
				const uint32 BulkDataFlags = (bDiscardBulkData ? 0 : BulkDataInlineFlags)
					| (bIsLODOptional ? BULKDATA_OptionalPayload : 0);

				StreamingBulkData.SetBulkDataFlags(BulkDataFlags);

				StreamingBulkData.Serialize(Ar, Owner, Idx);
				
				StreamingBulkData.ClearBulkDataFlags(0xffffffffu);
				StreamingBulkData.SetBulkDataFlags(OldBulkDataFlags);
			}
			else
#endif
			{
				StreamingBulkData.Serialize(Ar, Owner, Idx, false);
				bIsLODOptional = StreamingBulkData.IsOptional();

				const int64 BulkDataSize = StreamingBulkData.GetBulkDataSize();
				if (BulkDataSize == 0)
				{
					bDiscardBulkData = true;
					BuffersSize = 0;
				}
#if WITH_EDITORONLY_DATA
				else if (bUsingCookedEditorData && Ar.IsLoading())
				{
					// When using cooked data in editor, only the highest (lowest detailed) LOD is cooked inline, while the lower LODs are saved in StreamingBulkData.
					// So here we load StreamingBulkData from disk into TempData, then populate the render data from TempData.
					TArray<uint8> TempData;
					TempData.SetNum(BulkDataSize);
					void* Dest = TempData.GetData();
					StreamingBulkData.GetCopy(&Dest);

					FMemoryReader MemReader(TempData, true);
					MemReader.SetByteSwapping(Ar.IsByteSwapping());
					SerializeStreamedData(MemReader, OwnerMesh, Idx, ClassDataStripFlags, bNeedsCPUAccess, bForceKeepCPUResources);
				}
#endif
			}

			if (!bDiscardBulkData)
			{
				SerializeAvailabilityInfo(Ar, Idx, StripFlags.IsClassDataStripped(CDSF_AdjacencyData_DEPRECATED), bNeedsCPUAccess);
			}

			if (bInlined && Ar.IsLoading() && StreamingBulkData.GetBulkDataSize() > 0)
			{
				void* Data = StreamingBulkData.Lock(LOCK_READ_ONLY);
				FMemoryReaderView MemReader(MakeMemoryView(Data, StreamingBulkData.GetBulkDataSize()), true);
				MemReader.SetByteSwapping(Ar.IsByteSwapping());
				SerializeStreamedData(MemReader, OwnerMesh, Idx, ClassDataStripFlags, bNeedsCPUAccess, bForceKeepCPUResources);
				StreamingBulkData.Unlock();
			}
		}
		else if (bInlined)
		{
			SerializeStreamedData(Ar, OwnerMesh, Idx, ClassDataStripFlags, bNeedsCPUAccess, bForceKeepCPUResources);
			bIsLODOptional = false;
		}
	}

	if(Ar.IsLoading())
	{
		SetIsDataReady(true);
	}
}

int32 FSkeletalMeshLODRenderData::NumNonClothingSections() const
{
	int32 NumSections = RenderSections.Num();
	int32 Count = 0;

	for (int32 i = 0; i < NumSections; i++)
	{
		const FSkelMeshRenderSection& Section = RenderSections[i];

		// If we have found the start of the clothing section, return that index, since it is equal to the number of non-clothing entries.
		if (!Section.HasClothingData())
		{
			Count++;
		}
	}

	return Count;
}

uint32 FSkeletalMeshLODRenderData::FindSectionIndex(const FSkelMeshRenderSection& Section) const
{
	const FSkelMeshRenderSection* Start = RenderSections.GetData();

	if (Start == nullptr)
	{
		return -1;
	}

	uint32 Ret = &Section - Start;

	if (Ret >= (uint32)RenderSections.Num())
	{
		Ret = -1;
	}

	return Ret;
}

FSkinWeightVertexBuffer* FSkeletalMeshLODRenderData::GetSkinWeightProfileVertexBuffer(const FSkinWeightProfileStack& InProfileStack)
{
	const FSkinWeightProfileStack ProfileStack = InProfileStack.Normalized();
	if (!SkinWeightProfilesData.HasProfileStack(ProfileStack) && !ProfileStack.IsEmpty())
	{
		const bool bAllowCPU = FSkinWeightProfilesData::AllowCPU();
		const bool bHandleDelayedLoads = FSkinWeightProfilesData::HandleDelayedLoads();
		
		// If we have CPU-side weights available, use them right away.
		if (bAllowCPU &&
			((bHandleDelayedLoads && SkinWeightVertexBuffer.GetDataVertexBuffer()->IsWeightDataValid()) ||
			(!bHandleDelayedLoads && SkinWeightVertexBuffer.GetNeedsCPUAccess())))
		{
			SkinWeightProfilesData.InitialiseProfileBuffer(ProfileStack);
		}
		else
		{	
			using EReadbackStatus = FSkinWeightProfilesData::EReadbackStatus;
			const EReadbackStatus Status = SkinWeightProfilesData.GetReadbackStatus();
			SkinWeightProfilesData.PendingProfileStack = ProfileStack;
			if (Status != EReadbackStatus::BufferInitialized)
			{	
				// Even though the skin weight profile buffer has not been created here,
				// the GPU readback work is enqueued here. This function needs to be 
				// pumped until the buffer has been initialized. 
				SkinWeightProfilesData.PumpGPUReadback();

			}
		}
	}

	return SkinWeightProfilesData.GetOverrideBuffer(ProfileStack);
}

int32 FSkeletalMeshLODRenderData::GetTotalFaces() const
{
	int32 TotalFaces = 0;
	for (int32 i = 0; i < RenderSections.Num(); i++)
	{
		TotalFaces += RenderSections[i].NumTriangles;
	}

	return TotalFaces;
}

bool FSkeletalMeshLODRenderData::HasClothData() const
{
	for (int32 SectionIdx = 0; SectionIdx<RenderSections.Num(); SectionIdx++)
	{
		if (RenderSections[SectionIdx].HasClothingData())
		{
			return true;
		}
	}
	return false;
}

void FSkeletalMeshLODRenderData::GetSectionFromVertexIndex(int32 InVertIndex, int32& OutSectionIndex, int32& OutVertIndex) const
{
	OutSectionIndex = 0;
	OutVertIndex = 0;

	int32 VertCount = 0;

	// Iterate over each chunk
	for (int32 SectionCount = 0; SectionCount < RenderSections.Num(); SectionCount++)
	{
		const FSkelMeshRenderSection& Section = RenderSections[SectionCount];
		OutSectionIndex = SectionCount;

		// Is it in Soft vertex range?
		if (InVertIndex < VertCount + Section.GetNumVertices())
		{
			OutVertIndex = InVertIndex - VertCount;
			return;
		}
		VertCount += Section.NumVertices;
	}

	// InVertIndex should always be in some chunk!
	//check(false);
}
