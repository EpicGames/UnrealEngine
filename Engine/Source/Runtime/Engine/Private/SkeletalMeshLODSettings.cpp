// Copyright Epic Games, Inc. All Rights Reserved.

#include "Engine/SkeletalMeshLODSettings.h"
#include "Engine/DataAsset.h"
#include "Engine/SkeletalMesh.h"
#include "Engine/SkinnedAssetCommon.h"
#include "UObject/UObjectIterator.h"
#include "Animation/AnimSequence.h"
#include "UObject/FortniteMainBranchObjectVersion.h"
#include "Rendering/SkeletalMeshModel.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SkeletalMeshLODSettings)

DEFINE_LOG_CATEGORY_STATIC(LogSkeletalMeshLODSettings, Warning, All)

extern const TCHAR* GSkeletalMeshMinLodQualityLevelCVarName;
extern const TCHAR* GSkeletalMeshMinLodQualityLevelScalabilitySection;
extern const TCHAR* GNoRefBiasQualityLevelCVarName;
extern const TCHAR* GNoRefBiasQualityLevelScalabilitySection;

USkeletalMeshLODSettings::USkeletalMeshLODSettings(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bSupportLODStreaming.Default = false;
	MaxNumStreamedLODs.Default = 0;
	// TODO: support saving some but not all optional LODs
	MaxNumOptionalLODs.Default = 0;

	MinQualityLevelLod.SetQualityLevelCVarForCooking(GSkeletalMeshMinLodQualityLevelCVarName, GSkeletalMeshMinLodQualityLevelScalabilitySection);

	NoRefStreamingLODBias.Default = -1;
	NoRefStreamingLODBias.SetQualityLevelCVarForCooking(GNoRefBiasQualityLevelCVarName, GNoRefBiasQualityLevelScalabilitySection);
}

const FSkeletalMeshLODGroupSettings& USkeletalMeshLODSettings::GetSettingsForLODLevel(const int32 LODIndex) const
{
	if (LODIndex < LODGroups.Num())
	{
		return LODGroups[LODIndex];
	}

	// This should not happen as of right now, since the function is only called with 'Default' as name
	ensureMsgf(false, TEXT("Invalid Skeletal mesh default settings LOD Level"));

	// Default return value to prevent compile issue
	static FSkeletalMeshLODGroupSettings DefaultReturnValue;
	return DefaultReturnValue;
}


int32 USkeletalMeshLODSettings::GetNumberOfSettings() const
{
	return LODGroups.Num();
}

bool USkeletalMeshLODSettings::SetLODSettingsToMesh(USkeletalMesh* InMesh, int32 LODIndex) const
{
	if (InMesh->IsValidLODIndex(LODIndex) && LODGroups.IsValidIndex(LODIndex))
	{
		FSkeletalMeshLODInfo* LODInfo = InMesh->GetLODInfo(LODIndex);
		const FSkeletalMeshLODGroupSettings& Setting = LODGroups[LODIndex];
		LODInfo->ReductionSettings = Setting.ReductionSettings;
		LODInfo->ScreenSize = Setting.ScreenSize;
		LODInfo->LODHysteresis = Setting.LODHysteresis;
		LODInfo->WeightOfPrioritization = Setting.WeightOfPrioritization;
		LODInfo->bAllowMeshDeformer = Setting.bAllowMeshDeformer;
		// if we have available bake pose
		// it's possible for skeleton to be null if this happens in the middle of importing
		// so if skeleton is null, we allow copy (the GetBakePose will check correct skeleton when get it)
		if (Setting.BakePose && (!InMesh->GetSkeleton() || !Setting.BakePose->GetSkeleton()))
		{
			LODInfo->BakePose = Setting.BakePose;
		}
		else
		{
			LODInfo->BakePose = nullptr;
		}

		// select joints that mesh has
		// reset the list
		LODInfo->BonesToRemove.Reset();
		const FReferenceSkeleton& RefSkeleton = InMesh->GetRefSkeleton();

		// copy the shared setting to mesh setting
		// make sure we have joint in the mesh 
		LODInfo->BonesToPrioritize.Reset();
		for (auto& Bone : Setting.BonesToPrioritize)
		{
			if (RefSkeleton.FindBoneIndex(Bone) != INDEX_NONE)
			{
				LODInfo->BonesToPrioritize.Add(FBoneReference(Bone));
			}
		}

		// copy the shared setting to mesh setting
#if WITH_EDITOR
		const FSkeletalMeshLODModel* LodModel = InMesh->GetImportedModel()->LODModels.IsValidIndex(LODIndex) ? &(InMesh->GetImportedModel()->LODModels[LODIndex]) : nullptr;
#endif
		LODInfo->SectionsToPrioritize.Reset();
		for (const int32 SettingSectionIndex : Setting.SectionsToPrioritize)
		{
			int32 FinalSettingSectionIndex = SettingSectionIndex;
#if WITH_EDITOR
			if (LodModel)
			{
				//Make sure all sections to prioritize are valid LODModel sections
				FinalSettingSectionIndex = INDEX_NONE;
				for (int32 SectionIndex = 0; SectionIndex < LodModel->Sections.Num(); ++SectionIndex)
				{
					const FSkelMeshSection& Section = LodModel->Sections[SectionIndex];
					if (Section.ChunkedParentSectionIndex == INDEX_NONE && SettingSectionIndex == Section.OriginalDataSectionIndex)
					{
						FinalSettingSectionIndex = Section.OriginalDataSectionIndex;
						break;
					}
				}
			}
#endif
			if (FinalSettingSectionIndex != INDEX_NONE)
			{
				LODInfo->SectionsToPrioritize.AddUnique(FSectionReference(FinalSettingSectionIndex));
			}
		}

		if (Setting.BoneFilterActionOption == EBoneFilterActionOption::Remove)
		{
			for (int32 BoneListIndex = 0; BoneListIndex < Setting.BoneList.Num(); ++BoneListIndex)
			{
				const int32 BoneIndex = RefSkeleton.FindBoneIndex(Setting.BoneList[BoneListIndex].BoneName);
				// do we have the bone?
				if (BoneIndex != INDEX_NONE)
				{
					// if I'm including, this is all done, and good to go
					if (Setting.BoneList[BoneListIndex].bExcludeSelf == false)
					{
						LODInfo->BonesToRemove.Add(FBoneReference(Setting.BoneList[BoneListIndex].BoneName));
					}
					else // we add all children
					{
						TArray<int32> ChildBones;
						if (RefSkeleton.GetDirectChildBones(BoneIndex, ChildBones) > 0)
						{
							for (int32 ChildIndex = 0; ChildIndex < ChildBones.Num(); ++ChildIndex)
							{
								LODInfo->BonesToRemove.Add(FBoneReference(RefSkeleton.GetBoneName(ChildBones[ChildIndex])));
							}
						}
					}
				}
			}
		}
		else if (Setting.BoneFilterActionOption == EBoneFilterActionOption::Keep)
		{
			// add chain of that joint (all parents and itself)
			auto AddChain = [](const FReferenceSkeleton& InRefSkeleton, int32 InBoneIndex, TArray<int32>& InOutBoneIndices)
			{
				int32 BoneIndex = InBoneIndex;
				while (BoneIndex != INDEX_NONE)
				{
					InOutBoneIndices.AddUnique(BoneIndex);
					BoneIndex = InRefSkeleton.GetParentIndex(BoneIndex);
				}
			};

			TArray<int32> BoneIndices;
			// this operation is expensive. We keep all the list of joints that they want to keep
			// and then later remove joints that are not in the list
			for (int32 BoneListIndex = 0; BoneListIndex < Setting.BoneList.Num(); ++BoneListIndex)
			{
				const int32 BoneIndex = RefSkeleton.FindBoneIndex(Setting.BoneList[BoneListIndex].BoneName);
				if (BoneIndex != INDEX_NONE)
				{
					// since I'm included, just add chain of whole thing
					if (Setting.BoneList[BoneListIndex].bExcludeSelf == false)
					{
						AddChain(RefSkeleton, BoneIndex, BoneIndices);
					}
					else 
					{
						// since we're excluding itself, we give parent info
						int32 ParentIndex = RefSkeleton.GetParentIndex(BoneIndex);
						if (ParentIndex != INDEX_NONE)
						{
							AddChain(RefSkeleton, ParentIndex, BoneIndices);
						}
					}
				}
			}

			// now we have to go through, any joint that isn't child of these joints
			const TArray<FMeshBoneInfo>& MeshRefInfo = RefSkeleton.GetRefBoneInfo();
			for (int32 BoneIndex = 0; BoneIndex<MeshRefInfo.Num(); ++BoneIndex)
			{
				if (!BoneIndices.Contains(BoneIndex))
				{
					LODInfo->BonesToRemove.Add(FBoneReference(MeshRefInfo[BoneIndex].Name));
				}
			}
		}

		return true;
	}

	return false;
}

int32 USkeletalMeshLODSettings::SetLODSettingsToMesh(USkeletalMesh* InMesh) const
{
	if (InMesh)
	{
		InMesh->SetMinLod(MinLod);
		InMesh->SetQualityLevelMinLod(MinQualityLevelLod);
		InMesh->SetDisableBelowMinLodStripping(DisableBelowMinLodStripping);
		InMesh->SetNoRefStreamingLODBias(NoRefStreamingLODBias);
#if WITH_EDITORONLY_DATA
		InMesh->SetOverrideLODStreamingSettings(bOverrideLODStreamingSettings);
		InMesh->SetSupportLODStreaming(bSupportLODStreaming);
		InMesh->SetMaxNumStreamedLODs(MaxNumStreamedLODs);
		InMesh->SetMaxNumOptionalLODs(MaxNumOptionalLODs);
#endif
		// we only fill up until we have enough LODs
		const int32 NumSettings = FMath::Min(LODGroups.Num(), InMesh->GetLODNum());
		for (int32 Index = 0; Index < NumSettings; ++Index)
		{
			SetLODSettingsToMesh(InMesh, Index);
		}

		return NumSettings;
	}

	return 0;
}

int32 USkeletalMeshLODSettings::SetLODSettingsFromMesh(USkeletalMesh* InMesh)
{
	// in this case, we just set all settings to Mesh
	if (InMesh)
	{
		MinLod = InMesh->GetMinLod();
		MinQualityLevelLod = InMesh->GetQualityLevelMinLod();
		DisableBelowMinLodStripping = InMesh->GetDisableBelowMinLodStripping();
		NoRefStreamingLODBias = InMesh->GetNoRefStreamingLODBias();
#if WITH_EDITORONLY_DATA
		bOverrideLODStreamingSettings = InMesh->GetOverrideLODStreamingSettings();
		bSupportLODStreaming = InMesh->GetSupportLODStreaming();
		MaxNumStreamedLODs = InMesh->GetMaxNumStreamedLODs();
		MaxNumOptionalLODs = InMesh->GetMaxNumOptionalLODs();
#endif
		// we only fill up until we have enough LODs
		const int32 NumSettings = InMesh->GetLODNum();
		LODGroups.Reset(NumSettings);
		LODGroups.AddDefaulted(NumSettings);

		for (int32 Index = 0; Index < NumSettings; ++Index)
		{
			FSkeletalMeshLODInfo* LODInfo = InMesh->GetLODInfo(Index);
			FSkeletalMeshLODGroupSettings& Setting = LODGroups[Index];
			Setting.ReductionSettings = LODInfo->ReductionSettings;
			Setting.ScreenSize = LODInfo->ScreenSize;
			Setting.LODHysteresis = LODInfo->LODHysteresis;
			// copy mesh setting to shared setting
			Setting.BonesToPrioritize.Reset();
			for (auto& Bone : LODInfo->BonesToPrioritize)
			{
				Setting.BonesToPrioritize.Add(Bone.BoneName);
			}

#if WITH_EDITOR
			// copy the shared setting to mesh setting
			// make sure we have the section in the mesh 
			const FSkeletalMeshLODModel& LodModel = InMesh->GetImportedModel()->LODModels[Index];
#endif
			Setting.SectionsToPrioritize.Reset();
			for (const FSectionReference& SectionReference : LODInfo->SectionsToPrioritize)
			{
				int32 FinalSettingSectionIndex = SectionReference.SectionIndex;
#if WITH_EDITOR
				//In editor we can validate the prioritized sections
				if (!SectionReference.IsValidToEvaluate(LodModel))
				{
					FinalSettingSectionIndex = INDEX_NONE;
				}
#endif
				if (FinalSettingSectionIndex != INDEX_NONE)
				{
					Setting.SectionsToPrioritize.AddUnique(FinalSettingSectionIndex);
				}
			}

			Setting.WeightOfPrioritization = LODInfo->WeightOfPrioritization;
			Setting.BakePose = LODInfo->BakePose;
			Setting.BoneFilterActionOption = EBoneFilterActionOption::Remove;
			// select joints that mesh has
			// reset the list
			for (int32 BoneIndex = 0; BoneIndex < LODInfo->BonesToRemove.Num(); ++BoneIndex)
			{
				FBoneFilter NewFilter;
				NewFilter.bExcludeSelf = false;
				NewFilter.BoneName = LODInfo->BonesToRemove[BoneIndex].BoneName;
				Setting.BoneList.Add(NewFilter);
			}

			Setting.bAllowMeshDeformer = LODInfo->bAllowMeshDeformer;
		}

		return NumSettings;
	}

	return 0;
}

#if WITH_EDITOR
void USkeletalMeshLODSettings::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	// apply to all instance that is already loaded
	for (FThreadSafeObjectIterator Iter(USkeletalMesh::StaticClass()); Iter; ++Iter)
	{
		USkeletalMesh* Mesh = Cast<USkeletalMesh>(*Iter);
		// if PhysicsAssetOverride is NULL, it uses SkeletalMesh Physics Asset, so I'll need to update here
		if (Mesh->GetLODSettings() == this)
		{
			// apply the change
			SetLODSettingsToMesh(Mesh);
		}
	}
}
#endif // WITH_EDITOR

void USkeletalMeshLODSettings::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

	Ar.UsingCustomVersion(FFortniteMainBranchObjectVersion::GUID);
	Ar.UsingCustomVersion(FEditorObjectVersion::GUID);

	if (Ar.CustomVer(FFortniteMainBranchObjectVersion::GUID) < FFortniteMainBranchObjectVersion::ConvertReductionSettingOptions)
	{
		for (int32 Index = 0; Index < LODGroups.Num(); ++Index)
		{
			FSkeletalMeshOptimizationSettings& ReductionSettings = LODGroups[Index].ReductionSettings;
			// prior to this version, both of them were used
			ReductionSettings.ReductionMethod = SMOT_TriangleOrDeviation;
			if (ReductionSettings.MaxDeviationPercentage == 0.f)
			{
				// 0.f and 1.f should produce same result. However, it is bad to display 0.f in the slider
				// as 0.01 and 0.f causes extreme confusion. 
				ReductionSettings.MaxDeviationPercentage = 1.f;
			}
		}
	}
}

/////////////////////////////////////////////////////////////
// FSkeletalMeshOptimizationSettings 
/////////////////////////////////////////////////////////////
FSkeletalMeshOptimizationSettings FSkeletalMeshLODGroupSettings::GetReductionSettings() const
{
	return ReductionSettings;
}

const float FSkeletalMeshLODGroupSettings::GetScreenSize() const
{
	return ScreenSize.Default;
}

