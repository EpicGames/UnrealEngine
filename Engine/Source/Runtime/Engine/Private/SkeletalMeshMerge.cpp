// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	SkeletalMeshMerge.cpp: Unreal skeletal mesh merging implementation.
=============================================================================*/

#include "SkeletalMeshMerge.h"
#include "Algo/Unique.h"
#include "Animation/Skeleton.h"
#include "Engine/SkeletalMesh.h"
#include "Engine/SkeletalMeshSocket.h"
#include "Engine/SkinnedAssetCommon.h"
#include "EngineLogs.h"
#include "Rendering/SkeletalMeshRenderData.h"
#include "Materials/MaterialInterface.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SkeletalMeshMerge)

/*-----------------------------------------------------------------------------
	FSkeletalMeshMerge
-----------------------------------------------------------------------------*/

/**
* Constructor
* @param InMergeMesh - destination mesh to merge to
* @param InSrcMeshList - array of source meshes to merge
* @param InForceSectionMapping - optional array to map sections from the source meshes to merged section entries
*/
FSkeletalMeshMerge::FSkeletalMeshMerge(USkeletalMesh* InMergeMesh, 
									   const TArray<USkeletalMesh*>& InSrcMeshList, 
									   const TArray<FSkelMeshMergeSectionMapping>& InForceSectionMapping,
									   int32 InStripTopLODs,
                                       EMeshBufferAccess InMeshBufferAccess,
									   const FSkelMeshMergeUVTransformMapping* InSectionUVTransforms)
:	MergeMesh(InMergeMesh)
,	SrcMeshList(InSrcMeshList)
,	StripTopLODs(InStripTopLODs)
,   MeshBufferAccess(InMeshBufferAccess)
,	ForceSectionMapping(InForceSectionMapping)
,	SectionUVTransforms(InSectionUVTransforms)
{
	check(MergeMesh);
}


FSkeletalMeshMerge::FSkeletalMeshMerge(USkeletalMesh* InMergeMesh, 
										const TArray<USkeletalMesh*>& InSrcMeshList, 
										const TArray<FSkelMeshMergeSectionMapping>& InForceSectionMapping,
										int32 InStripTopLODs,
										EMeshBufferAccess InMeshBufferAccess,
										PRAGMA_DISABLE_DEPRECATION_WARNINGS
										FSkelMeshMergeUVTransforms* InSectionUVTransforms)
										PRAGMA_ENABLE_DEPRECATION_WARNINGS
:	MergeMesh(InMergeMesh)
,	SrcMeshList(InSrcMeshList)
,	StripTopLODs(InStripTopLODs)
,   MeshBufferAccess(InMeshBufferAccess)
,	ForceSectionMapping(InForceSectionMapping)
PRAGMA_DISABLE_DEPRECATION_WARNINGS
,	SectionUVTransforms(&DummySectionUVTransforms)
PRAGMA_ENABLE_DEPRECATION_WARNINGS
{
	if (InSectionUVTransforms)
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		for (int32 MeshIdx = 0; MeshIdx < InSrcMeshList.Num(); ++MeshIdx)
		{
			if (MeshIdx < InSectionUVTransforms->UVTransformsPerMesh.Num())
			{
				const TArray<FTransform>& OldPerMeshTransforms = InSectionUVTransforms->UVTransformsPerMesh[MeshIdx];
				FSkelMeshMergeMeshUVTransforms& NewPerMeshTransforms = DummySectionUVTransforms.UVTransformsPerMesh.AddDefaulted_GetRef();
				NewPerMeshTransforms.UVTransforms = OldPerMeshTransforms;
			}
		}
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}
}

/** Helper macro to call GenerateLODModel which requires compile time vertex type. */
#define GENERATE_LOD_MODEL( VertexType, NumUVs ) \
{\
	switch( NumUVs )\
	{\
	case 1:\
		GenerateLODModel< VertexType<1> >( LODIdx + StripTopLODs );\
		break;\
	case 2:\
		GenerateLODModel< VertexType<2> >( LODIdx + StripTopLODs );\
		break;\
	case 3:\
		GenerateLODModel< VertexType<3> >( LODIdx + StripTopLODs );\
		break;\
	case 4:\
		GenerateLODModel< VertexType<4> >( LODIdx + StripTopLODs );\
		break;\
	default:\
		checkf(false, TEXT("Invalid number of UV sets.  Must be between 0 and 4") );\
		break;\
	}\
}\

/**
* Merge/Composite the list of source meshes onto the merge one
* The MergeMesh is reinitialized 
* @return true if succeeded
*/
bool FSkeletalMeshMerge::DoMerge(TArray<FRefPoseOverride>* RefPoseOverrides /* = nullptr */)
{
	MergeSkeleton(RefPoseOverrides);

	return FinalizeMesh();
}

void FSkeletalMeshMerge::MergeSkeleton(const TArray<FRefPoseOverride>* RefPoseOverrides /* = nullptr */)
{
	// Release the rendering resources.

	MergeMesh->ReleaseResources();
	MergeMesh->ReleaseResourcesFence.Wait();

	// Build the reference skeleton & sockets.

	BuildReferenceSkeleton(SrcMeshList, NewRefSkeleton, MergeMesh->GetSkeleton());

	// Assign new referencer skeleton.
	MergeMesh->SetRefSkeleton(NewRefSkeleton);

	BuildSockets(SrcMeshList);

	// Override the reference bone poses & sockets, if specified.

	if (RefPoseOverrides)
	{
		OverrideReferenceSkeletonPose(*RefPoseOverrides, NewRefSkeleton, MergeMesh->GetSkeleton());
		OverrideMergedSockets(*RefPoseOverrides);
	}

	// Rebuild inverse ref pose matrices here as some access patterns 
	// may need to access these matrices before FinalizeMesh is called
	// (which would *normally* rebuild the inv ref matrices).
	MergeMesh->GetRefBasesInvMatrix().Empty();
	MergeMesh->CalculateInvRefMatrices();
}

bool FSkeletalMeshMerge::FinalizeMesh()
{
	bool Result = true;

	// Find the common maximum number of LODs available in the list of source meshes.

	int32 MaxNumLODs = CalculateLodCount(SrcMeshList);

	if (MaxNumLODs == -1)
	{
		UE_LOGF(LogSkeletalMesh, Warning, "FSkeletalMeshMerge: Invalid source mesh list");
		return false;
	}

	ReleaseResources(MaxNumLODs);

	// Create a mapping from each input mesh bone to bones in the merged mesh.

	SrcMeshInfo.Empty();
	SrcMeshInfo.AddZeroed(SrcMeshList.Num());

	for (int32 MeshIdx = 0; MeshIdx < SrcMeshList.Num(); MeshIdx++)
	{
		USkeletalMesh* SrcMesh = SrcMeshList[MeshIdx];
		if (SrcMesh)
		{
			if (SrcMesh->GetHasVertexColors())
			{
				MergeMesh->SetHasVertexColors(true);
#if WITH_EDITORONLY_DATA
				MergeMesh->SetVertexColorGuid(FGuid::NewGuid());
#endif
			}

			FMergeMeshInfo& MeshInfo = SrcMeshInfo[MeshIdx];
			MeshInfo.SrcToDestRefSkeletonMap.AddUninitialized(SrcMesh->GetRefSkeleton().GetRawBoneNum());

			for (int32 i = 0; i < SrcMesh->GetRefSkeleton().GetRawBoneNum(); i++)
			{
				FName SrcBoneName = SrcMesh->GetRefSkeleton().GetBoneName(i);
				int32 DestBoneIndex = NewRefSkeleton.FindBoneIndex(SrcBoneName);

				if (DestBoneIndex == INDEX_NONE)
				{
					// Missing bones shouldn't be possible, but can happen with invalid meshes;
					// map any bone we are missing to the 'root'.

					DestBoneIndex = 0;
				}

				MeshInfo.SrcToDestRefSkeletonMap[i] = DestBoneIndex;
			}
		}
	}

	// If things are going ok so far...
	if (Result)
	{

		// Array of per-lod number of UV sets
		TArray<uint32> PerLODNumUVSets;
		TArray<uint32> PerLODMaxBoneInfluences;
		TArray<bool> PerLODUse16BitBoneIndex;
		PerLODNumUVSets.AddZeroed(MaxNumLODs);
		PerLODMaxBoneInfluences.AddZeroed(MaxNumLODs);
		PerLODUse16BitBoneIndex.AddZeroed(MaxNumLODs);

		// Get the number of UV sets for each LOD.
		for (int32 MeshIdx = 0; MeshIdx < SrcMeshList.Num(); MeshIdx++)
		{
			USkeletalMesh* SrcSkelMesh = SrcMeshList[MeshIdx];
			FSkeletalMeshRenderData* SrcResource = SrcSkelMesh->GetResourceForRendering();

			for (int32 LODIdx = 0; LODIdx < MaxNumLODs; LODIdx++)
			{
				if (SrcResource->LODRenderData.IsValidIndex(LODIdx))
				{
					uint32& NumUVSets = PerLODNumUVSets[LODIdx];
					NumUVSets = FMath::Max(NumUVSets, SrcResource->LODRenderData[LODIdx].GetNumTexCoords());

					PerLODMaxBoneInfluences[LODIdx] = FMath::Max(PerLODMaxBoneInfluences[LODIdx], SrcResource->LODRenderData[LODIdx].GetVertexBufferMaxBoneInfluences());
					PerLODUse16BitBoneIndex[LODIdx] |= SrcResource->LODRenderData[LODIdx].DoesVertexBufferUse16BitBoneIndex();
				}
			}
		}

		// process each LOD for the new merged mesh
		MergeMesh->AllocateResourceForRendering();
		for (int32 LODIdx = 0; LODIdx < MaxNumLODs; LODIdx++)
		{
			const FSkeletalMeshLODInfo* LODInfoPtr = MergeMesh->GetLODInfo(LODIdx);
			bool bUseFullPrecisionUVs = LODInfoPtr ? LODInfoPtr->BuildSettings.bUseFullPrecisionUVs : false;
			if (!bUseFullPrecisionUVs)
			{
				GENERATE_LOD_MODEL(TGPUSkinVertexFloat16Uvs, PerLODNumUVSets[LODIdx]);
			}
			else
			{
				GENERATE_LOD_MODEL(TGPUSkinVertexFloat32Uvs, PerLODNumUVSets[LODIdx]);
			}
		}

		// Initialize the render data's unified bone map from the LOD storage.
		FSkeletalMeshRenderData* MergeResource = MergeMesh->GetResourceForRendering();
		if (MergeResource)
		{
			MergeResource->InitUnifiedBoneMapFromLODs();
		}

		// update the merge skel mesh entries
		if (!ProcessMergeMesh())
		{
			Result = false;
		}

		// If in game, streaming must be disabled as there are no files to stream from.
		// In editor, the engine can stream from the DDC and create the required files on cook.
		if (!GIsEditor)
		{
			MergeMesh->NeverStream = true;
		}

		// Reinitialize the mesh's render resources.
		MergeMesh->InitResources();
	}

	return Result;
}

/**
* Merge a bonemap with an existing bonemap and keep track of remapping
* (a bonemap is a list of indices of bones in the USkeletalMesh::RefSkeleton array)
* @param MergedBoneMap - out merged bonemap
* @param BoneMapToMergedBoneMap - out of mapping from original bonemap to new merged bonemap 
* @param BoneMap - input bonemap to merge
*/
void FSkeletalMeshMerge::MergeBoneMap( TArray<FBoneIndexType>& MergedBoneMap, TArray<FBoneIndexType>& BoneMapToMergedBoneMap, const TArray<FBoneIndexType>& BoneMap )
{
	BoneMapToMergedBoneMap.AddUninitialized( BoneMap.Num() );
	for( int32 IdxB=0; IdxB < BoneMap.Num(); IdxB++ )
	{
		BoneMapToMergedBoneMap[IdxB] = MergedBoneMap.AddUnique( BoneMap[IdxB] );
	}
}

static void BoneMapToNewRefSkel(const TArray<FBoneIndexType>& InBoneMap, const TArray<int32>& SrcToDestRefSkeletonMap, TArray<FBoneIndexType>& OutBoneMap)
{
	OutBoneMap.Empty();
	OutBoneMap.AddUninitialized(InBoneMap.Num());

	for(int32 i=0; i<InBoneMap.Num(); i++)
	{
		check(InBoneMap[i] < SrcToDestRefSkeletonMap.Num());
		OutBoneMap[i] = SrcToDestRefSkeletonMap[InBoneMap[i]];
	}
}

/**
* Generate the list of sections that need to be created along with info needed to merge sections
* @param NewSectionArray - out array to populate
* @param LODIdx - current LOD to process
*/
void FSkeletalMeshMerge::GenerateNewSectionArray( TArray<FNewSectionInfo>& NewSectionArray, int32 LODIdx )
{

	NewSectionArray.Empty();
	for( int32 MeshIdx=0; MeshIdx < SrcMeshList.Num(); MeshIdx++ )
	{
		// source mesh
		USkeletalMesh* SrcMesh = SrcMeshList[MeshIdx];

		if( SrcMesh )
		{
			FSkeletalMeshRenderData* SrcResource = SrcMesh->GetResourceForRendering();
			int32 SourceLODIdx = FMath::Min(LODIdx, SrcResource->LODRenderData.Num()-1);
			FSkeletalMeshLODRenderData& SrcLODData = SrcResource->LODRenderData[SourceLODIdx];
			FSkeletalMeshLODInfo& SrcLODInfo = *(SrcMesh->GetLODInfo(SourceLODIdx));

			// iterate over each section of this LOD
			for( int32 SectionIdx=0; SectionIdx < SrcLODData.RenderSections.Num(); SectionIdx++ )
			{
				int32 MaterialId = -1;
				// check for the optional list of material ids corresponding to the list of src meshes
				// if the id is valid (not -1) it is used to find an existing section entry to merge with
				if( ForceSectionMapping.Num() == SrcMeshList.Num() &&
					ForceSectionMapping.IsValidIndex(MeshIdx) &&
					ForceSectionMapping[MeshIdx].SectionIDs.IsValidIndex(SectionIdx) )
				{
					MaterialId = ForceSectionMapping[MeshIdx].SectionIDs[SectionIdx];
				}

				FSkelMeshRenderSection& Section = SrcLODData.RenderSections[SectionIdx];

				// Convert Chunk.BoneMap from src to dest bone indices
				TArray<FBoneIndexType> DestChunkBoneMap;
				BoneMapToNewRefSkel(Section.BoneMap, SrcMeshInfo[MeshIdx].SrcToDestRefSkeletonMap, DestChunkBoneMap);


				// get the material for this section
				int32 MaterialIndex = Section.MaterialIndex;
				// use the remapping of material indices if there is a valid value
				if(SrcLODInfo.LODMaterialMap.IsValidIndex(SectionIdx) && SrcLODInfo.LODMaterialMap[SectionIdx] != INDEX_NONE && SrcMesh->GetMaterials().Num() > 0)
				{
					MaterialIndex = FMath::Clamp<int32>( SrcLODInfo.LODMaterialMap[SectionIdx], 0, SrcMesh->GetMaterials().Num() - 1);
				}

				const FSkeletalMaterial& SkeletalMaterial = SrcMesh->GetMaterials()[MaterialIndex];
				UMaterialInterface* Material = SkeletalMaterial.MaterialInterface;

				// see if there is an existing entry in the array of new sections that matches its material
				// if there is a match then the source section can be added to its list of sections to merge 
				int32 FoundIdx = INDEX_NONE;
				for( int32 Idx=0; Idx < NewSectionArray.Num(); Idx++ )
				{
					FNewSectionInfo& NewSectionInfo = NewSectionArray[Idx];
					// check for a matching material or a matching material index id if it is valid
					if( (MaterialId == -1 && Material == NewSectionInfo.Material) ||
						(MaterialId != -1 && MaterialId == NewSectionInfo.MaterialId) )
					{
						check(NewSectionInfo.MergeSections.Num());

						// merge the bonemap from the source section with the existing merged bonemap
						TArray<FBoneIndexType> TempMergedBoneMap(NewSectionInfo.MergedBoneMap);
						TArray<FBoneIndexType> TempBoneMapToMergedBoneMap;
						MergeBoneMap(TempMergedBoneMap,TempBoneMapToMergedBoneMap,DestChunkBoneMap);

						TArray<FTransform> SrcUVTransform;
						if (SectionUVTransforms != nullptr && MeshIdx < SectionUVTransforms->UVTransformsPerMesh.Num())
						{
							SrcUVTransform = SectionUVTransforms->UVTransformsPerMesh[MeshIdx].UVTransforms;
						}

						// add the source section as a new merge entry
						FMergeSectionInfo& MergeSectionInfo = *new(NewSectionInfo.MergeSections) FMergeSectionInfo(
							SrcMesh,
							&SrcLODData.RenderSections[SectionIdx],
							SrcUVTransform
							);
						// keep track of remapping for the existing chunk's bonemap 
						// so that the bone matrix indices can be updated for the vertices
						MergeSectionInfo.BoneMapToMergedBoneMap = TempBoneMapToMergedBoneMap;

						// use the updated bonemap for this new section
						NewSectionInfo.MergedBoneMap = TempMergedBoneMap;

						// keep track of the entry that was found
						FoundIdx = Idx;
						break;
					}
				}

				// new section entries will be created if the material for the source section was not found
				if( FoundIdx == INDEX_NONE )
				{
					// create a new section entry
					const FName& MaterialSlotName = SkeletalMaterial.MaterialSlotName;
					const FMeshUVChannelInfo& UVChannelData = SkeletalMaterial.UVChannelData;
					FNewSectionInfo& NewSectionInfo = *new(NewSectionArray) FNewSectionInfo(Material, MaterialId, MaterialSlotName, UVChannelData);
					// initialize the merged bonemap to simply use the original chunk bonemap
					NewSectionInfo.MergedBoneMap = DestChunkBoneMap;

					TArray<FTransform> SrcUVTransform;
					if (SectionUVTransforms != nullptr && MeshIdx < SectionUVTransforms->UVTransformsPerMesh.Num())
					{
						SrcUVTransform = SectionUVTransforms->UVTransformsPerMesh[MeshIdx].UVTransforms;
					}
					// add a new merge section entry
					FMergeSectionInfo& MergeSectionInfo = *new(NewSectionInfo.MergeSections) FMergeSectionInfo(
						SrcMesh,
						&SrcLODData.RenderSections[SectionIdx],
						SrcUVTransform);
					// since merged bonemap == chunk.bonemap then remapping is just pass-through
					MergeSectionInfo.BoneMapToMergedBoneMap.Empty( DestChunkBoneMap.Num() );
					for( int32 i=0; i < DestChunkBoneMap.Num(); i++ )
					{
						MergeSectionInfo.BoneMapToMergedBoneMap.Add((FBoneIndexType)i);
					}
				}
			}
		}
	}
}

template<typename VertexDataType>
void FSkeletalMeshMerge::CopyVertexFromSource(VertexDataType& DestVert, const FSkeletalMeshLODRenderData& SrcLODData, int32 SourceVertIdx, const FMergeSectionInfo& MergeSectionInfo)
{
	DestVert.Position = SrcLODData.StaticVertexBuffers.PositionVertexBuffer.VertexPosition(SourceVertIdx);
	DestVert.TangentX = SrcLODData.StaticVertexBuffers.StaticMeshVertexBuffer.VertexTangentX(SourceVertIdx);
	DestVert.TangentZ = SrcLODData.StaticVertexBuffers.StaticMeshVertexBuffer.VertexTangentZ(SourceVertIdx);

	// Copy all UVs that are available
	uint32 LODNumTexCoords = SrcLODData.StaticVertexBuffers.StaticMeshVertexBuffer.GetNumTexCoords();
	const uint32 ValidLoopCount = FMath::Min(VertexDataType::NumTexCoords, LODNumTexCoords);
	for (uint32 UVIndex = 0; UVIndex < ValidLoopCount; ++UVIndex)
	{
		FVector2D UVs = FVector2D(SrcLODData.StaticVertexBuffers.StaticMeshVertexBuffer.GetVertexUV_Typed<VertexDataType::StaticMeshVertexUVType>(SourceVertIdx, UVIndex));
		if (UVIndex < (uint32)MergeSectionInfo.UVTransforms.Num())
		{
			FVector Transformed = MergeSectionInfo.UVTransforms[UVIndex].TransformPosition(FVector(UVs, 1.f));
			UVs = FVector2D(Transformed.X, Transformed.Y);
		}
		DestVert.UVs[UVIndex] = FVector2f(UVs);	// LWC_TODO: Precision loss
	}
	
	// now just fill up zero value if we didn't reach till end
	for (uint32 UVIndex = ValidLoopCount; UVIndex < VertexDataType::NumTexCoords; ++UVIndex)
	{
		DestVert.UVs[UVIndex] = FVector2f::ZeroVector;
	}
}

/**
* Creates a new LOD model and adds the new merged sections to it. Modifies the MergedMesh.
* @param LODIdx - current LOD to process
*/
template<typename VertexDataType>
void FSkeletalMeshMerge::GenerateLODModel( int32 LODIdx )
{
	// add the new LOD model entry
	FSkeletalMeshRenderData* MergeResource = MergeMesh->GetResourceForRendering();
	check(MergeResource);

	FSkeletalMeshLODRenderData& MergeLODData = *new FSkeletalMeshLODRenderData;
	MergeResource->LODRenderData.Add(&MergeLODData);
	// add the new LOD info entry
	FSkeletalMeshLODInfo& MergeLODInfo = MergeMesh->AddLODInfo();
	MergeLODInfo.ScreenSize = MergeLODInfo.LODHysteresis = UE_MAX_FLT;

	// generate an array with info about new sections that need to be created
	TArray<FNewSectionInfo> NewSectionArray;
	GenerateNewSectionArray( NewSectionArray, LODIdx );

	uint32 MaxIndex = 0;

	// merged vertex buffer
	TArray< VertexDataType > MergedVertexBuffer;
	// merged skin weight buffer
	TArray< FSkinWeightInfo > MergedSkinWeightBuffer;
	// merged vertex color buffer
	TArray< FColor > MergedColorBuffer;
	// merged index buffer
	TArray<uint32> MergedIndexBuffer;

	// The total number of UV sets for this LOD model
	uint32 TotalNumUVs = 0;

	uint32 SourceMaxBoneInfluences = 0;
	bool bSourceUse16BitBoneIndex = false;

	for( int32 CreateIdx=0; CreateIdx < NewSectionArray.Num(); CreateIdx++ )
	{
		FNewSectionInfo& NewSectionInfo = NewSectionArray[CreateIdx];

		// ActiveBoneIndices contains all the bones used by the verts from all the sections of this LOD model
		// Add the bones used by this new section
		for( int32 Idx=0; Idx < NewSectionInfo.MergedBoneMap.Num(); Idx++ )
		{
			MergeLODData.ActiveBoneIndices.AddUnique( NewSectionInfo.MergedBoneMap[Idx] );
		}

		// add the new section entry
		FSkelMeshRenderSection& Section = *new(MergeLODData.RenderSections) FSkelMeshRenderSection;

		// set the new bonemap from the merged sections
		// these are the bones that will be used by this new section
		Section.BoneMap = NewSectionInfo.MergedBoneMap;

		// init vert totals
		Section.NumVertices = 0;

		// keep track of the current base vertex for this section in the merged vertex buffer
		Section.BaseVertexIndex = MergedVertexBuffer.Num();

		// find existing material index
		check(MergeMesh->GetMaterials().Num() == MaterialIds.Num());
		int32 MatIndex;
		if(NewSectionInfo.MaterialId == -1)
		{
			MatIndex = MergeMesh->GetMaterials().Find(NewSectionInfo.Material);
		}
		else
		{
			MatIndex = MaterialIds.Find(NewSectionInfo.MaterialId);
		}

		// if it doesn't exist, make new entry
		if(MatIndex == INDEX_NONE)
		{
			FSkeletalMaterial SkeletalMaterial(NewSectionInfo.Material, true, false, NewSectionInfo.SlotName);
			SkeletalMaterial.UVChannelData = NewSectionInfo.UVChannelData;
			MergeMesh->GetMaterials().Add(SkeletalMaterial);
			MaterialIds.Add(NewSectionInfo.MaterialId);
			Section.MaterialIndex = MergeMesh->GetMaterials().Num()-1;
		}
		else
		{
			Section.MaterialIndex = MatIndex;
		}
		
		// init tri totals
		Section.NumTriangles = 0;
		// keep track of the current base index for this section in the merged index buffer
		Section.BaseIndex = MergedIndexBuffer.Num();

		FMeshUVChannelInfo& MergedUVData = MergeMesh->GetMaterials()[Section.MaterialIndex].UVChannelData;

		// iterate over all of the sections that need to be merged together
		for( int32 MergeIdx=0; MergeIdx < NewSectionInfo.MergeSections.Num(); MergeIdx++ )
		{
			FMergeSectionInfo& MergeSectionInfo = NewSectionInfo.MergeSections[MergeIdx];
			int32 SourceLODIdx = FMath::Min(LODIdx, MergeSectionInfo.SkelMesh->GetResourceForRendering()->LODRenderData.Num()-1);

			// Take the max UV density for each UVChannel between all sections that are being merged.
			const int32 NewSectionMatId = MergeSectionInfo.Section->MaterialIndex;
			if(MergeSectionInfo.SkelMesh->GetMaterials().IsValidIndex(NewSectionMatId))
			{
				const FMeshUVChannelInfo& NewSectionUVData = MergeSectionInfo.SkelMesh->GetMaterials()[NewSectionMatId].UVChannelData;
				for (int32 i = 0; i < MAX_TEXCOORDS; i++)
				{
					const float NewSectionUVDensity = NewSectionUVData.LocalUVDensities[i];
					float& UVDensity = MergedUVData.LocalUVDensities[i];

					UVDensity = FMath::Max(UVDensity, NewSectionUVDensity);
				}
			}

			// get the source skel LOD info from this merge entry
			const FSkeletalMeshLODInfo& SrcLODInfo = *(MergeSectionInfo.SkelMesh->GetLODInfo(SourceLODIdx));

			// keep track of the lowest LOD displayfactor and hysteresis
			MergeLODInfo.ScreenSize.Default = FMath::Min<float>(MergeLODInfo.ScreenSize.Default, SrcLODInfo.ScreenSize.Default);
#if WITH_EDITORONLY_DATA
			for(const TPair<FName, float>& PerPlatform : SrcLODInfo.ScreenSize.PerPlatform)
			{
				float* Value = MergeLODInfo.ScreenSize.PerPlatform.Find(PerPlatform.Key);
				if(Value)
				{
					*Value = FMath::Min<float>(PerPlatform.Value, *Value);	
				}
				else
				{
					MergeLODInfo.ScreenSize.PerPlatform.Add(PerPlatform.Key, PerPlatform.Value);
				}
			}
#endif
			MergeLODInfo.BuildSettings.bUseFullPrecisionUVs |= SrcLODInfo.BuildSettings.bUseFullPrecisionUVs;
			MergeLODInfo.BuildSettings.bUseBackwardsCompatibleF16TruncUVs |= SrcLODInfo.BuildSettings.bUseBackwardsCompatibleF16TruncUVs;
			MergeLODInfo.BuildSettings.bUseHighPrecisionTangentBasis |= SrcLODInfo.BuildSettings.bUseHighPrecisionTangentBasis;

			MergeLODInfo.LODHysteresis = FMath::Min<float>(MergeLODInfo.LODHysteresis,SrcLODInfo.LODHysteresis);

			// get the source skel LOD model from this merge entry
			const FSkeletalMeshLODRenderData& SrcLODData = MergeSectionInfo.SkelMesh->GetResourceForRendering()->LODRenderData[SourceLODIdx];

			// add required bones from this source model entry to the merge model entry
			for( int32 Idx=0; Idx < SrcLODData.RequiredBones.Num(); Idx++ )
			{
				FName SrcLODBoneName = MergeSectionInfo.SkelMesh->GetRefSkeleton().GetBoneName(SrcLODData.RequiredBones[Idx] );
				int32 MergeBoneIndex = NewRefSkeleton.FindBoneIndex(SrcLODBoneName);
				
				if (MergeBoneIndex != INDEX_NONE)
				{
					MergeLODData.RequiredBones.AddUnique(MergeBoneIndex);
				}
			}

			// update vert total
			Section.NumVertices += MergeSectionInfo.Section->NumVertices;

			// update total number of vertices 
			const int32 NumTotalVertices = MergeSectionInfo.Section->NumVertices;

			// add the vertices from the original source mesh to the merged vertex buffer					
			const int32 MaxVertIdx = FMath::Min<int32>( 
				MergeSectionInfo.Section->BaseVertexIndex + NumTotalVertices,
				SrcLODData.StaticVertexBuffers.PositionVertexBuffer.GetNumVertices()
				);

			const int32 MaxColorIdx = SrcLODData.StaticVertexBuffers.ColorVertexBuffer.GetNumVertices();

			// update max number of influences
			const uint32 MaxBoneInfluences = SrcLODData.GetSkinWeightVertexBuffer()->GetMaxBoneInfluences();
			const bool bUse16BitBoneIndex = SrcLODData.GetSkinWeightVertexBuffer()->Use16BitBoneIndex();

			SourceMaxBoneInfluences = FMath::Max(SourceMaxBoneInfluences, MaxBoneInfluences);
			bSourceUse16BitBoneIndex |= bUse16BitBoneIndex;

			// update RenderSection number of max influences
			Section.MaxBoneInfluences = MaxBoneInfluences;
			
			// update total number of TexCoords
			const uint32 LODNumTexCoords = SrcLODData.StaticVertexBuffers.StaticMeshVertexBuffer.GetNumTexCoords();
			if (TotalNumUVs < LODNumTexCoords)
			{
				TotalNumUVs = LODNumTexCoords;
			}

			// keep track of the current base vertex index before adding any new vertices
			// this will be needed to remap the index buffer values to the new range
			const int32 CurrentBaseVertexIndex = MergedVertexBuffer.Num();
			
			for( int32 VertIdx=MergeSectionInfo.Section->BaseVertexIndex; VertIdx < MaxVertIdx; VertIdx++ )
			{
				// add the new vertex
				VertexDataType& DestVert = MergedVertexBuffer[MergedVertexBuffer.AddUninitialized()];
				FSkinWeightInfo& DestWeight = MergedSkinWeightBuffer[MergedSkinWeightBuffer.AddUninitialized()];

				CopyVertexFromSource<VertexDataType>(DestVert, SrcLODData, VertIdx, MergeSectionInfo);

				DestWeight = SrcLODData.GetSkinWeightVertexBuffer()->GetVertexSkinWeights(VertIdx);

				// if the mesh uses vertex colors, copy the source color if possible or default to white
				if( MergeMesh->GetHasVertexColors() )
				{
					if( VertIdx < MaxColorIdx )
					{
						const FColor& SrcColor = SrcLODData.StaticVertexBuffers.ColorVertexBuffer.VertexColor(VertIdx);
						MergedColorBuffer.Add(SrcColor);
					}
					else
					{
						const FColor ColorWhite(255, 255, 255);
						MergedColorBuffer.Add(ColorWhite);
					}
				}

				// remap the bone index used by this vertex to match the mergedbonemap 
				for( uint32 Idx=0; Idx < MAX_TOTAL_INFLUENCES; Idx++ )
				{
					if (DestWeight.InfluenceWeights[Idx] > 0)
					{
						checkSlow(MergeSectionInfo.BoneMapToMergedBoneMap.IsValidIndex(DestWeight.InfluenceBones[Idx]));
						DestWeight.InfluenceBones[Idx] = (FBoneIndexType)MergeSectionInfo.BoneMapToMergedBoneMap[DestWeight.InfluenceBones[Idx]];
					}
				}
			}

			// update total number of triangles
			Section.NumTriangles += MergeSectionInfo.Section->NumTriangles;

			// add the indices from the original source mesh to the merged index buffer					
			const int32 MaxIndexIdx = FMath::Min<int32>( 
				MergeSectionInfo.Section->BaseIndex + MergeSectionInfo.Section->NumTriangles * 3, 
				SrcLODData.MultiSizeIndexContainer.GetIndexBuffer()->Num()
				);
            for (int32 IndexIdx = MergeSectionInfo.Section->BaseIndex; IndexIdx < MaxIndexIdx; IndexIdx++)
            {
                uint32 SrcIndex = SrcLODData.MultiSizeIndexContainer.GetIndexBuffer()->Get(IndexIdx);

                // add offset to each index to match the new entries in the merged vertex buffer
                checkSlow(SrcIndex >= MergeSectionInfo.Section->BaseVertexIndex);
                uint32 DstIndex = SrcIndex - MergeSectionInfo.Section->BaseVertexIndex + CurrentBaseVertexIndex;
                checkSlow(DstIndex < (uint32)MergedVertexBuffer.Num());

                // add the new index to the merged vertex buffer
                MergedIndexBuffer.Add(DstIndex);
                if (MaxIndex < DstIndex)
                {
                    MaxIndex = DstIndex;
                }

            }

            {
                if (MergeSectionInfo.Section->DuplicatedVerticesBuffer.bHasOverlappingVertices)
                {
                    if (Section.DuplicatedVerticesBuffer.bHasOverlappingVertices)
                    {
                        // Merge
                        int32 StartIndex = Section.DuplicatedVerticesBuffer.DupVertData.Num();
                        int32 StartVertex = Section.DuplicatedVerticesBuffer.DupVertIndexData.Num();
                        Section.DuplicatedVerticesBuffer.DupVertData.ResizeBuffer(StartIndex + MergeSectionInfo.Section->DuplicatedVerticesBuffer.DupVertData.Num());
                        Section.DuplicatedVerticesBuffer.DupVertIndexData.ResizeBuffer(Section.NumVertices);
                    
                        uint8* VertData = Section.DuplicatedVerticesBuffer.DupVertData.GetDataPointer();
                        uint8* IndexData = Section.DuplicatedVerticesBuffer.DupVertIndexData.GetDataPointer();
                        for (int32 i = StartIndex; i < Section.DuplicatedVerticesBuffer.DupVertData.Num(); ++i)
                        {
                            *((uint32*)(VertData + i * sizeof(uint32))) += CurrentBaseVertexIndex - MergeSectionInfo.Section->BaseVertexIndex;
                        }
                        for (uint32 i = StartVertex; i < Section.NumVertices; ++i)
                        {
                            ((FIndexLengthPair*)(IndexData + i * sizeof(FIndexLengthPair)))->Index += StartIndex;
                        }
                    }
                    else
                    {
                        Section.DuplicatedVerticesBuffer.DupVertData = MergeSectionInfo.Section->DuplicatedVerticesBuffer.DupVertData;
                        Section.DuplicatedVerticesBuffer.DupVertIndexData = MergeSectionInfo.Section->DuplicatedVerticesBuffer.DupVertIndexData;
                        uint8* VertData = Section.DuplicatedVerticesBuffer.DupVertData.GetDataPointer();
                        for (int32 i = 0; i < Section.DuplicatedVerticesBuffer.DupVertData.Num(); ++i)
                        {
                            *((uint32*)(VertData + i * sizeof(uint32))) += CurrentBaseVertexIndex - MergeSectionInfo.Section->BaseVertexIndex;
                        }
                    }
                    Section.DuplicatedVerticesBuffer.bHasOverlappingVertices = true;
                }
                else
                {
                    Section.DuplicatedVerticesBuffer.DupVertData.ResizeBuffer(1);
                    Section.DuplicatedVerticesBuffer.DupVertIndexData.ResizeBuffer(Section.NumVertices);

                    uint8* VertData = Section.DuplicatedVerticesBuffer.DupVertData.GetDataPointer();
                    uint8* IndexData = Section.DuplicatedVerticesBuffer.DupVertIndexData.GetDataPointer();
                    
                    FMemory::Memzero(IndexData, Section.NumVertices * sizeof(FIndexLengthPair));
                    FMemory::Memzero(VertData, sizeof(uint32));
                }
            }
		}
	}

	// If any source mesh uses unified bone indices, unify the merged output's bone maps too.
	// This ensures the merged mesh is compatible with GPU instance LOD and single-pass skinning.
	{
		bool bAnySourceHasUnifiedBoneMap = false;
		for (USkeletalMesh* SrcMesh : SrcMeshList)
		{
			if (SrcMesh && SrcMesh->GetResourceForRendering() && SrcMesh->GetResourceForRendering()->HasUnifiedBoneMap())
			{
				bAnySourceHasUnifiedBoneMap = true;
				break;
			}
		}

		if (bAnySourceHasUnifiedBoneMap)
		{
			// Build unified bone map from the union of all merged section bone maps.
			TArray<FBoneIndexType> UnifiedBoneMap;
			for (const FSkelMeshRenderSection& Section : MergeLODData.RenderSections)
			{
				UnifiedBoneMap.Append(Section.BoneMap);
			}
			UnifiedBoneMap.Sort();
			UnifiedBoneMap.SetNum(Algo::Unique(UnifiedBoneMap));

			// Remap each section's vertex bone indices from per-section bone map space to unified space,
			// then overwrite the section's bone map with the unified map.
			for (FSkelMeshRenderSection& Section : MergeLODData.RenderSections)
			{
				// Build remap: OldBoneMapIndex -> UnifiedBoneMapIndex
				TArray<uint32> RemapTable;
				RemapTable.SetNumUninitialized(Section.BoneMap.Num());
				for (int32 OldIndex = 0; OldIndex < Section.BoneMap.Num(); ++OldIndex)
				{
					const int32 UnifiedIndex = UnifiedBoneMap.Find(Section.BoneMap[OldIndex]);
					check(UnifiedIndex != INDEX_NONE);
					RemapTable[OldIndex] = (uint32)UnifiedIndex;
				}

				// Remap vertex bone indices in the merged vertex buffer for this section's vertices.
				for (uint32 VertIdx = 0; VertIdx < Section.NumVertices; ++VertIdx)
				{
					FSkinWeightInfo& Weight = MergedSkinWeightBuffer[Section.BaseVertexIndex + VertIdx];

					for (int32 InfluenceIndex = 0; InfluenceIndex < MAX_TOTAL_INFLUENCES; ++InfluenceIndex)
					{
						if (Weight.InfluenceWeights[InfluenceIndex] > 0)
						{
							Weight.InfluenceBones[InfluenceIndex] = (FBoneIndexType)RemapTable[Weight.InfluenceBones[InfluenceIndex]];
						}
					}
				}

				Section.BoneMap = UnifiedBoneMap;
				Section.bUnifiedBoneMap = true;
			}

			// Set the unified bone map on the LOD render data (points into the first section's BoneMap storage).
			if (!MergeLODData.RenderSections.IsEmpty())
			{
				MergeLODData.UnifiedBoneMap = MergeLODData.RenderSections[0].BoneMap;
			}
		}
	}

    const bool bNeedsCPUAccess = (MeshBufferAccess == EMeshBufferAccess::ForceCPUAndGPU);

	// sort required bone array in strictly increasing order
	MergeLODData.RequiredBones.Sort();
	MergeMesh->GetRefSkeleton().EnsureParentsExistAndSort(MergeLODData.ActiveBoneIndices);
	
	// copy the new vertices and indices to the vertex buffer for the new model
	MergeLODData.StaticVertexBuffers.StaticMeshVertexBuffer.SetUseFullPrecisionUVs(MergeLODInfo.BuildSettings.bUseFullPrecisionUVs);

	MergeLODData.StaticVertexBuffers.PositionVertexBuffer.Init(MergedVertexBuffer.Num(), bNeedsCPUAccess);
	MergeLODData.StaticVertexBuffers.StaticMeshVertexBuffer.Init(MergedVertexBuffer.Num(), TotalNumUVs, bNeedsCPUAccess);

	bool bUseBackwardsCompatibleF16TruncUVs = MergeLODInfo.BuildSettings.bUseBackwardsCompatibleF16TruncUVs;

	for (int i = 0; i < MergedVertexBuffer.Num(); i++)
	{
		MergeLODData.StaticVertexBuffers.PositionVertexBuffer.VertexPosition(i) = MergedVertexBuffer[i].Position;
		MergeLODData.StaticVertexBuffers.StaticMeshVertexBuffer.SetVertexTangents(i, MergedVertexBuffer[i].TangentX.ToFVector3f(), MergedVertexBuffer[i].GetTangentY(), MergedVertexBuffer[i].TangentZ.ToFVector3f());
		for (uint32 j = 0; j < TotalNumUVs; j++)
		{
			MergeLODData.StaticVertexBuffers.StaticMeshVertexBuffer.SetVertexUV(i, j, MergedVertexBuffer[i].UVs[j], bUseBackwardsCompatibleF16TruncUVs);
		}
	}

	MergeLODData.SkinWeightVertexBuffer.SetMaxBoneInfluences(SourceMaxBoneInfluences);
	MergeLODData.SkinWeightVertexBuffer.SetUse16BitBoneIndex(bSourceUse16BitBoneIndex);
	MergeLODData.SkinWeightVertexBuffer.SetNeedsCPUAccess(bNeedsCPUAccess);

	// copy vertex resource arrays
	MergeLODData.SkinWeightVertexBuffer = MergedSkinWeightBuffer;

	if( MergeMesh->GetHasVertexColors() )
	{
		MergeLODData.StaticVertexBuffers.ColorVertexBuffer.InitFromColorArray(MergedColorBuffer);
	}

	
	const uint8 DataTypeSize = (MaxIndex < MAX_uint16) ? sizeof(uint16) : sizeof(uint32);
	MergeLODData.MultiSizeIndexContainer.RebuildIndexBuffer(DataTypeSize, MergedIndexBuffer);
}

/**
* (Re)initialize and merge skeletal mesh info from the list of source meshes to the merge mesh
* @return true if succeeded
*/
bool FSkeletalMeshMerge::ProcessMergeMesh()
{
	bool Result=true;
	
	// copy settings and bone info from src meshes
	bool bNeedsInit=true;

	for( int32 MeshIdx=0; MeshIdx < SrcMeshList.Num(); MeshIdx++ )
	{
		USkeletalMesh* SrcMesh = SrcMeshList[MeshIdx];
		if( SrcMesh )
		{
			if( bNeedsInit )
			{
				// initialize the merged mesh with the first src mesh entry used
				MergeMesh->SetImportedBounds(SrcMesh->GetImportedBounds());
				// only initialize once
				bNeedsInit = false;
			}
			else
			{
				// add bounds
				MergeMesh->SetImportedBounds(MergeMesh->GetImportedBounds() + SrcMesh->GetImportedBounds());
			}
		}
	}
	// Rebuild inverse ref pose matrices.
	MergeMesh->GetRefBasesInvMatrix().Empty();
	MergeMesh->CalculateInvRefMatrices();

	return Result;
}

int32 FSkeletalMeshMerge::CalculateLodCount(const TArray<USkeletalMesh*>& SourceMeshList) const
{
	int32 LodCount = INT_MAX;

	for (int32 i = 0, MeshCount = SourceMeshList.Num(); i < MeshCount; ++i)
	{
		USkeletalMesh* SourceMesh = SourceMeshList[i];

		if (SourceMesh)
		{
			LodCount = FMath::Min<int32>(LodCount, SourceMesh->GetLODNum());
		}
	}

	if (LodCount == INT_MAX)
	{
		return -1;
	}

	// Decrease the number of LODs we are going to make based on StripTopLODs.
	// But, make sure there is at least one.

	LodCount -= StripTopLODs;
	LodCount = FMath::Max(LodCount, 1);
	
	return LodCount;
}

void FSkeletalMeshMerge::BuildReferenceSkeleton(const TArray<USkeletalMesh*>& SourceMeshList, FReferenceSkeleton& RefSkeleton, const USkeleton* SkeletonAsset)
{
	RefSkeleton.Empty();

	// Iterate through all the source mesh reference skeletons and compose the merged reference skeleton.

	FReferenceSkeletonModifier RefSkelModifier(RefSkeleton, SkeletonAsset);

	for (int32 MeshIndex = 0; MeshIndex < SourceMeshList.Num(); ++MeshIndex)
	{
		USkeletalMesh* SourceMesh = SourceMeshList[MeshIndex];

		if (!SourceMesh)
		{
			continue;
		}

		// Initialise new RefSkeleton with first mesh.

		if (RefSkeleton.GetRawBoneNum() == 0)
		{
			RefSkeleton = SourceMesh->GetRefSkeleton();
			continue;
		}

		// For subsequent meshes, add any missing bones.

		for (int32 i = 1; i < SourceMesh->GetRefSkeleton().GetRawBoneNum(); ++i)
		{
			FName SourceBoneName = SourceMesh->GetRefSkeleton().GetBoneName(i);
			int32 TargetBoneIndex = RefSkeleton.FindRawBoneIndex(SourceBoneName);

			// If the source bone is present in the new RefSkeleton, we skip it.

			if (TargetBoneIndex != INDEX_NONE)
			{
				continue;
			}

			// Add the source bone to the RefSkeleton.

			int32 SourceParentIndex = SourceMesh->GetRefSkeleton().GetParentIndex(i);
			FName SourceParentName = SourceMesh->GetRefSkeleton().GetBoneName(SourceParentIndex);
			int32 TargetParentIndex = RefSkeleton.FindRawBoneIndex(SourceParentName);

			if (TargetParentIndex == INDEX_NONE)
			{
				continue;
			}

			FMeshBoneInfo MeshBoneInfo = SourceMesh->GetRefSkeleton().GetRefBoneInfo()[i];
			MeshBoneInfo.ParentIndex = TargetParentIndex;

			RefSkelModifier.Add(MeshBoneInfo, SourceMesh->GetRefSkeleton().GetRefBonePose()[i]);
		}
	}
}

void FSkeletalMeshMerge::OverrideReferenceSkeletonPose(const TArray<FRefPoseOverride>& PoseOverrides, FReferenceSkeleton& TargetSkeleton, const USkeleton* SkeletonAsset)
{
	for (int32 i = 0, PoseMax = PoseOverrides.Num(); i < PoseMax; ++i)
	{
		const FRefPoseOverride& PoseOverride = PoseOverrides[i];
		const FReferenceSkeleton& SourceSkeleton = PoseOverride.SkeletalMesh->GetRefSkeleton();

		FReferenceSkeletonModifier RefSkelModifier(TargetSkeleton, SkeletonAsset);

		for (int32 j = 0, BoneMax = PoseOverride.Overrides.Num(); j < BoneMax; ++j)
		{
			const FName& BoneName = PoseOverride.Overrides[j].BoneName;
			int32 SourceBoneIndex = SourceSkeleton.FindBoneIndex(BoneName);

			if (SourceBoneIndex != INDEX_NONE)
			{
				bool bOverrideBone = (PoseOverride.Overrides[j].OverrideMode == FRefPoseOverride::ChildrenOnly) ? false : true;

				if (bOverrideBone)
				{
					OverrideReferenceBonePose(SourceBoneIndex, SourceSkeleton, RefSkelModifier);
				}

				bool bOverrideChildren = (PoseOverride.Overrides[j].OverrideMode == FRefPoseOverride::BoneOnly) ? false : true;

				if (bOverrideChildren)
				{
					for (int32 ChildBoneIndex = SourceBoneIndex + 1; ChildBoneIndex < SourceSkeleton.GetRawBoneNum(); ++ChildBoneIndex)
					{
						if (SourceSkeleton.BoneIsChildOf(ChildBoneIndex, SourceBoneIndex))
						{
							OverrideReferenceBonePose(ChildBoneIndex, SourceSkeleton, RefSkelModifier);
						}
					}
				}
			}
		}
	}
}

bool FSkeletalMeshMerge::OverrideReferenceBonePose(int32 SourceBoneIndex, const FReferenceSkeleton& SourceSkeleton, FReferenceSkeletonModifier& TargetSkeleton)
{
	FName BoneName = SourceSkeleton.GetBoneName(SourceBoneIndex);
	int32 TargetBoneIndex = TargetSkeleton.GetReferenceSkeleton().FindBoneIndex(BoneName);

	if (TargetBoneIndex != INDEX_NONE)
	{
		const FTransform& SourceBoneTransform = SourceSkeleton.GetRefBonePose()[SourceBoneIndex];
		TargetSkeleton.UpdateRefPoseTransform(TargetBoneIndex, SourceBoneTransform);

		return true;
	}

	return false;
}

void FSkeletalMeshMerge::ReleaseResources(int32 Slack)
{
	FSkeletalMeshRenderData* Resource = MergeMesh->GetResourceForRendering();
	if (Resource)
	{
		Resource->LODRenderData.Empty(Slack);
	}

	MergeMesh->SetNumSourceModels(0);
	MergeMesh->GetMaterials().Empty();
}

bool FSkeletalMeshMerge::AddSocket(const USkeletalMeshSocket* NewSocket, bool bIsSkeletonSocket)
{
	TArray<TObjectPtr<USkeletalMeshSocket>>& MergeMeshSockets = MergeMesh->GetMeshOnlySocketList();

	// Verify the socket doesn't already exist in the current Mesh list.
	for (USkeletalMeshSocket const * const ExistingSocket : MergeMeshSockets)
	{
		if (ExistingSocket->SocketName == NewSocket->SocketName)
		{
			return false;
		}
	}

	// The Skeleton will only be valid in cases where the passed in mesh already had a skeleton
	// (i.e. an existing mesh was used, or a created mesh was explicitly assigned a skeleton).
	// In either case, we want to avoid adding sockets to the Skeleton (as it is shared), but we
	// still need to check against it to prevent duplication.
	if (bIsSkeletonSocket && MergeMesh->GetSkeleton())
	{
		if (MergeMesh->GetSkeleton()->Sockets.Num() > 0)
		{
			return false;
		}
	}

	USkeletalMeshSocket* NewSocketDuplicate = CastChecked<USkeletalMeshSocket>(StaticDuplicateObject(NewSocket, MergeMesh));
	MergeMeshSockets.Add(NewSocketDuplicate);

	return true;
}

void FSkeletalMeshMerge::AddSockets(const TArray<USkeletalMeshSocket*>& NewSockets, bool bAreSkeletonSockets)
{
	for (USkeletalMeshSocket* NewSocket : NewSockets)
	{
		AddSocket(NewSocket, bAreSkeletonSockets);
	}
}

void FSkeletalMeshMerge::BuildSockets(const TArray<USkeletalMesh*>& SourceMeshList)
{
	TArray<TObjectPtr<USkeletalMeshSocket>>& MeshSocketList = MergeMesh->GetMeshOnlySocketList();
	MeshSocketList.Empty();

	// Iterate through the all the source MESH sockets, only adding the new sockets.

	for (USkeletalMesh const * const SourceMesh : SourceMeshList)
	{
		if (SourceMesh)
		{
			const TArray<USkeletalMeshSocket*>& NewMeshSocketList = SourceMesh->GetMeshOnlySocketList();
			AddSockets(NewMeshSocketList, false);
		}
	}

	// Iterate through the all the source SKELETON sockets, only adding the new sockets.

	for (USkeletalMesh const * const SourceMesh : SourceMeshList)
	{
		if (SourceMesh && SourceMesh->GetSkeleton())
		{
			const TArray<USkeletalMeshSocket*>& NewSkeletonSocketList = SourceMesh->GetSkeleton()->Sockets;
			AddSockets(NewSkeletonSocketList, true);
		}
	}

	MergeMesh->RebuildSocketMap();
}

void FSkeletalMeshMerge::OverrideSocket(const USkeletalMeshSocket* SourceSocket)
{
	TArray<TObjectPtr<USkeletalMeshSocket>>& SocketList = MergeMesh->GetMeshOnlySocketList();

	for (int32 i = 0, SocketCount = SocketList.Num(); i < SocketCount; ++i)
	{
		USkeletalMeshSocket* TargetSocket = SocketList[i];

		if (TargetSocket->SocketName == SourceSocket->SocketName)
		{
			TargetSocket->BoneName = SourceSocket->BoneName;
			TargetSocket->RelativeLocation = SourceSocket->RelativeLocation;
			TargetSocket->RelativeRotation = SourceSocket->RelativeRotation;
			TargetSocket->RelativeScale = SourceSocket->RelativeScale;
		}
	}
}

void FSkeletalMeshMerge::OverrideBoneSockets(const FName& BoneName, const TArray<USkeletalMeshSocket*>& SourceSocketList)
{
	for (int32 i = 0, SourceSocketCount = SourceSocketList.Num(); i < SourceSocketCount; ++i)
	{
		const USkeletalMeshSocket* SourceSocket = SourceSocketList[i];

		if (SourceSocket->BoneName == BoneName)
		{
			OverrideSocket(SourceSocket);
		}
	}
}

void FSkeletalMeshMerge::OverrideMergedSockets(const TArray<FRefPoseOverride>& PoseOverrides)
{
	for (int32 i = 0, PoseMax = PoseOverrides.Num(); i < PoseMax; ++i)
	{
		const FRefPoseOverride& PoseOverride = PoseOverrides[i];
		const FReferenceSkeleton& SourceSkeleton = PoseOverride.SkeletalMesh->GetRefSkeleton();
		
		//We assume the skeleton is valid here
		if (!ensure(PoseOverride.SkeletalMesh->GetSkeleton()))
		{
			continue;
		}

		const TArray<USkeletalMeshSocket*>& SkeletonSocketList = PoseOverride.SkeletalMesh->GetSkeleton()->Sockets;
		const TArray<USkeletalMeshSocket*>& MeshSocketList = const_cast<USkeletalMesh*>(PoseOverride.SkeletalMesh)->GetMeshOnlySocketList();

		for (int32 j = 0, BoneMax = PoseOverride.Overrides.Num(); j < BoneMax; ++j)
		{
			const FName& BoneName = PoseOverride.Overrides[j].BoneName;
			int32 SourceBoneIndex = SourceSkeleton.FindBoneIndex(BoneName);

			if (SourceBoneIndex != INDEX_NONE)
			{
				bool bOverrideBone = (PoseOverride.Overrides[j].OverrideMode == FRefPoseOverride::ChildrenOnly) ? false : true;

				if (bOverrideBone)
				{
					OverrideBoneSockets(BoneName, SkeletonSocketList);
					OverrideBoneSockets(BoneName, MeshSocketList);
				}

				bool bOverrideChildren = (PoseOverride.Overrides[j].OverrideMode == FRefPoseOverride::BoneOnly) ? false : true;

				if (bOverrideChildren)
				{
					for (int32 ChildBoneIndex = SourceBoneIndex + 1; ChildBoneIndex < SourceSkeleton.GetRawBoneNum(); ++ChildBoneIndex)
					{
						if (SourceSkeleton.BoneIsChildOf(ChildBoneIndex, SourceBoneIndex))
						{
							FName ChildBoneName = SourceSkeleton.GetBoneName(ChildBoneIndex);

							OverrideBoneSockets(ChildBoneName, SkeletonSocketList);
							OverrideBoneSockets(ChildBoneName, MeshSocketList);
						}
					}
				}
			}
		}
	}

	MergeMesh->RebuildSocketMap();
}

