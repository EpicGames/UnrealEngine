// Copyright Epic Games, Inc. All Rights Reserved.

#include "Rendering/SkeletalMeshModel.h"
#include "UObject/FortniteMainBranchObjectVersion.h"

#if WITH_EDITOR
#include "Engine/SkinnedAsset.h"
#include "Engine/SkinnedAssetCommon.h"
#include "Serialization/MemoryWriter.h"
#include "EngineUtils.h"
#include "SkeletalMeshLegacyCustomVersions.h"
#include "UObject/UE5MainStreamObjectVersion.h"

FSkeletalMeshModel::FSkeletalMeshModel()
	: bGuidIsHash(false)
{
}

void FSkeletalMeshModel::Serialize(FArchive& Ar, USkinnedAsset* Owner)
{
	DECLARE_SCOPE_CYCLE_COUNTER(TEXT("FSkeletalMeshModel::Serialize"), STAT_SkeletalMeshModel_Serialize, STATGROUP_LoadTime);
	Ar.UsingCustomVersion(FFortniteMainBranchObjectVersion::GUID);
	Ar.UsingCustomVersion(FSkeletalMeshCustomVersion::GUID);
	Ar.UsingCustomVersion(FUE5MainStreamObjectVersion::GUID);

	bool bIsEditorDataStripped = false;
	if (Ar.IsSaving() || (Ar.IsLoading() && Ar.CustomVer(FFortniteMainBranchObjectVersion::GUID) >= FFortniteMainBranchObjectVersion::AllowSkeletalMeshToReduceTheBaseLOD))
	{
		FStripDataFlags StripFlags(Ar);
		bIsEditorDataStripped = StripFlags.IsEditorDataStripped();
	}

	LODModels.Serialize(Ar, Owner);

	// For old content without a GUID, generate one now from the data
	if (Ar.IsLoading() && Ar.CustomVer(FSkeletalMeshCustomVersion::GUID) < FSkeletalMeshCustomVersion::SplitModelAndRenderData)
	{
		GenerateGUIDFromHash(Owner);
	}
	// Serialize the GUID
	else
	{
		Ar << SkeletalMeshModelGUID;
		Ar << bGuidIsHash;
	}

	if (!bIsEditorDataStripped)
	{
		if (Ar.IsLoading()
			&& (Ar.CustomVer(FFortniteMainBranchObjectVersion::GUID) >= FFortniteMainBranchObjectVersion::AllowSkeletalMeshToReduceTheBaseLOD)
			&& (Ar.CustomVer(FUE5MainStreamObjectVersion::GUID) < FUE5MainStreamObjectVersion::ConvertReductionBaseSkeletalMeshBulkDataToInlineReductionCacheData))
		{
			FReductionBaseSkeletalMeshBulkData::Serialize(Ar, OriginalReductionSourceMeshData_DEPRECATED, Owner);

			InlineReductionCacheDatas.Reserve(OriginalReductionSourceMeshData_DEPRECATED.Num());
			for (int32 LodIndex = 0; LodIndex < OriginalReductionSourceMeshData_DEPRECATED.Num(); ++LodIndex)
			{
				if (!LODModels.IsValidIndex(LodIndex))
				{
					break;
				}
				bool bNeedToUseReductionData = true;
				if (const FSkeletalMeshLODInfo* LODInfo = Owner->GetLODInfo(LodIndex))
				{
					if (!LODInfo->bHasBeenSimplified)
					{
						bNeedToUseReductionData = false;
					}
				}
				if (bNeedToUseReductionData)
				{
					if (FReductionBaseSkeletalMeshBulkData* ReductionBaseSkeletalMeshBulkData = OriginalReductionSourceMeshData_DEPRECATED[LodIndex])
					{
						uint32 LodVertexCount = 0;
						uint32 LodTriangleCount = 0;
						ReductionBaseSkeletalMeshBulkData->GetGeometryInfo(LodVertexCount, LodTriangleCount, Owner);
						InlineReductionCacheDatas.AddDefaulted_GetRef().SetCacheGeometryInfo(LodVertexCount, LodTriangleCount);
					}
				}
				else
				{
					InlineReductionCacheDatas.AddDefaulted_GetRef().SetCacheGeometryInfo(LODModels[LodIndex]);
				}
			}
		}

		if (Ar.IsSaving() || (Ar.IsLoading() && Ar.CustomVer(FUE5MainStreamObjectVersion::GUID) >= FUE5MainStreamObjectVersion::ConvertReductionBaseSkeletalMeshBulkDataToInlineReductionCacheData))
		{
			Ar << InlineReductionCacheDatas;
		}

		if (Ar.IsLoading())
		{
			//Add any missing cache
			if (InlineReductionCacheDatas.Num() < LODModels.Num())
			{
				InlineReductionCacheDatas.AddDefaulted(LODModels.Num() - InlineReductionCacheDatas.Num());
				for (int32 LodIndex = InlineReductionCacheDatas.Num(); LodIndex < LODModels.Num(); ++LodIndex)
				{
					InlineReductionCacheDatas[LodIndex].SetCacheGeometryInfo(LODModels[LodIndex]);
				}
			}
			else if (InlineReductionCacheDatas.Num() > LODModels.Num())
			{
				//If we have too much entry simply shrink the array to valid LODModel size
				InlineReductionCacheDatas.SetNum(LODModels.Num());
			}
		}
	}
}

void FSkeletalMeshModel::GenerateNewGUID()
{
	SkeletalMeshModelGUID = FGuid::NewGuid();
	bGuidIsHash = false;
}


void FSkeletalMeshModel::GenerateGUIDFromHash(USkinnedAsset* Owner)
{
	// Build the hash from the path name + the contents of the bulk data.
	FSHA1 Sha;
	TArray<TCHAR, FString::AllocatorType> OwnerName = Owner->GetPathName().GetCharArray();
	Sha.Update((uint8*)OwnerName.GetData(), OwnerName.Num() * OwnerName.GetTypeSize());

	TArray<uint8> TempBytes;
	FMemoryWriter Ar(TempBytes, /*bIsPersistent=*/ true);
	LODModels.Serialize(Ar, Owner);

	if (TempBytes.Num() > 0)
	{
		Sha.Update(TempBytes.GetData(), TempBytes.Num() * sizeof(uint8));
	}
	Sha.Final();

	// Retrieve the hash and use it to construct a pseudo-GUID.
	uint32 Hash[5];
	Sha.GetHash((uint8*)Hash);
	SkeletalMeshModelGUID = FGuid(Hash[0] ^ Hash[4], Hash[1], Hash[2], Hash[3]);

	bGuidIsHash = true;
}

FString FSkeletalMeshModel::GetIdString() const
{
	FString GuidString = SkeletalMeshModelGUID.ToString();
	if (bGuidIsHash)
	{
		GuidString += TEXT("X");
	}
	return GuidString;
}

void FSkeletalMeshModel::SyncronizeLODUserSectionsData()
{
	for (int32 LODIndex = 0; LODIndex < LODModels.Num(); ++LODIndex)
	{
		FSkeletalMeshLODModel& Model = LODModels[LODIndex];
		Model.SyncronizeUserSectionsDataArray();
	}
}

FString FSkeletalMeshModel::GetLODModelIdString() const
{
	FSHA1 Sha;
	for (int32 LODIndex = 0; LODIndex < LODModels.Num(); ++LODIndex)
	{
		const FSkeletalMeshLODModel& Model = LODModels[LODIndex];
		const FString BuildStringID = Model.GetLODModelDeriveDataKey();
		Model.BuildStringID = BuildStringID;
		TArray<TCHAR, FString::AllocatorType> IDArray = BuildStringID.GetCharArray();
		Sha.Update((uint8*)IDArray.GetData(), IDArray.Num() * IDArray.GetTypeSize());
	}
	Sha.Final();
	// Retrieve the hash and use it to construct a pseudo-GUID.
	uint32 Hash[5];
	Sha.GetHash((uint8*)Hash);
	return FGuid(Hash[0] ^ Hash[4], Hash[1], Hash[2], Hash[3]).ToString(EGuidFormats::Digits);
}

void FSkeletalMeshModel::GetResourceSizeEx(FResourceSizeEx& CumulativeResourceSize)
{
	for (int32 LODIndex = 0; LODIndex < LODModels.Num(); ++LODIndex)
	{
		const FSkeletalMeshLODModel& Model = LODModels[LODIndex];
		Model.GetResourceSizeEx(CumulativeResourceSize);
	}
}

void FSkeletalMeshModel::EmptyOriginalReductionSourceMeshData()
{
	for (FReductionBaseSkeletalMeshBulkData* ReductionData : OriginalReductionSourceMeshData_DEPRECATED)
	{
		ReductionData->EmptyBulkData();
		delete ReductionData;
	}
	OriginalReductionSourceMeshData_DEPRECATED.Empty();
}

#endif // WITH_EDITOR
