// Copyright Epic Games, Inc. All Rights Reserved.

#include "Rendering/SkeletalMeshRenderData.h"
#include "Engine/SkinnedAsset.h"
#include "Rendering/SkeletalMeshModel.h"
#include "Rendering/NaniteResources.h"
#include "Engine/SkeletalMesh.h"
#include "Engine/SkinnedAssetAsyncCompileUtils.h"
#include "Engine/SkinnedAssetCommon.h"
#include "EngineLogs.h"
#include "UObject/Package.h"
#include "Rendering/RenderCommandPipes.h"
#include "Rendering/RayTracingGeometryManager.h"
#include "Algo/Unique.h"
#include "SkeletalMeshAttributes.h"

#if WITH_EDITOR
#include "MeshUtilities.h"
#include "Modules/ModuleManager.h"
#include "ProfilingDebugging/CookStats.h"
#include "DerivedDataCacheInterface.h"
#include "IMeshBuilderModule.h"
#include "RenderingThread.h"
#include "Serialization/LargeMemoryReader.h"
#include "Serialization/LargeMemoryWriter.h"

#if ENABLE_COOK_STATS
namespace SkeletalMeshCookStats
{
	static FCookStats::FDDCResourceUsageStats UsageStats;
	static FCookStatsManager::FAutoRegisterCallback RegisterCookStats([](FCookStatsManager::AddStatFuncRef AddStat)
	{
		UsageStats.LogStats(AddStat, TEXT("SkeletalMesh.Usage"), TEXT(""));
	});
}
#endif

extern int32 GStripSkeletalMeshLodsDuringCooking;

#endif // WITH_EDITOR

int32 GSkeletalMeshKeepMobileMinLODSettingOnDesktop = 0;
static FAutoConsoleVariableRef CVarSkeletalMeshKeepMobileMinLODSettingOnDesktop(
	TEXT("r.SkeletalMesh.KeepMobileMinLODSettingOnDesktop"),
	GSkeletalMeshKeepMobileMinLODSettingOnDesktop,
	TEXT("If non-zero, mobile setting for MinLOD will be stored in the cooked data for desktop platforms"));

#if WITH_EDITOR

/** 
 * Utility functions for storing and accessing data that exceeds the usual signed 32bit limits 
 * for data length.
 * We achieve this by splitting the data into multiple chunks that the DDC can handle along with 
 * a header chunk. Then when the data is requested we can load each chunk and reconstruct the 
 * original data.
 */
namespace DDCUtils64Bit
{
	struct FDDCChunkingHeader
	{
		/** Overall size of the data when reconstructed. */
		int64 TotalSize;
		/** The number of chunks that the data was split into. */
		int32 NumChunks;
	};

	/** The same as calling GetDerivedDataCacheRef().GetSynchronous(...) but with a TArray64 as the output parameter. */
	bool GetSynchronous(const FString& DerivedDataKey, USkinnedAsset* Owner, TArray64<uint8>& OutDerivedData)
	{
		TStringBuilder<512> OwnerPathName;
		Owner->GetPathName(nullptr, OwnerPathName);

		TArray<uint8> Data32Bit;
		if (GetDerivedDataCacheRef().GetSynchronous(*DerivedDataKey, Data32Bit, OwnerPathName))
		{
			// Note that currently the MoveTemp does nothing and the data is being copied although 
			// at some point this might be optimized and the TArray64 will just assume ownership of
			// the TArrays allocation instead.
			OutDerivedData = MoveTemp(Data32Bit);
			return true;
		}
		else
		{
			TStringBuilder<512> HeaderKey;
			HeaderKey << DerivedDataKey << TEXT("Header");

			TArray<uint8> HeaderData;
			HeaderData.Reserve(sizeof(FDDCChunkingHeader));

			// Early out if we cannot find the header or that it is the wrong size (in which case we cannot cast it)
			if (!GetDerivedDataCacheRef().GetSynchronous(HeaderKey.ToString(), HeaderData, Owner->GetPathName()) || HeaderData.Num() != sizeof(FDDCChunkingHeader))
			{
				return false;
			}

			FDDCChunkingHeader* Header = (FDDCChunkingHeader*)HeaderData.GetData();
			OutDerivedData.Reserve(Header->TotalSize);

			for (int32 ChunkIndex = 0; ChunkIndex < Header->NumChunks; ChunkIndex++)
			{
				TStringBuilder<512> ChunkKey;
				ChunkKey << DerivedDataKey << TEXT("Chunk") << ChunkIndex;

				TArray<uint8> ChunkData;
				if (!GetDerivedDataCacheRef().GetSynchronous(ChunkKey.ToString(), ChunkData, OwnerPathName))
				{
					OutDerivedData.Empty(); // Get rid of any partial results we might have
					return false;
				}

				OutDerivedData.Append(ChunkData);
			}

			return true;
		}
	}

	/** The same as calling GetDerivedDataCacheRef().Put(...) but with a TArrayView64 as the input data. */
	void Put(const FString& DerivedDataKey, USkinnedAsset* Owner, TArrayView64<const uint8> DerivedData)
	{
		TStringBuilder<512> OwnerPathName;
		Owner->GetPathName(nullptr, OwnerPathName);

		// We don't use the full 32 bit range as internally the DDC might append info to the end of 
		// the chunk, so we reserve 4kb for this, which is more than enough space to be safe.

		const int64 ChunkSize = MAX_int32 - (4 * 1024);
		if (DerivedData.Num() <= ChunkSize)
		{
			TArrayView<const uint8> DerivedData32Bit(DerivedData.GetData(), (int32)DerivedData.Num());
			GetDerivedDataCacheRef().Put(*DerivedDataKey, DerivedData32Bit, OwnerPathName);
		}
		else
		{
			const int32 NumChunks = (int32)FMath::DivideAndRoundUp(DerivedData.Num(), ChunkSize);

			FDDCChunkingHeader Header{ DerivedData.Num(), NumChunks };

			{
				TStringBuilder<512> HeaderKey;
				HeaderKey << DerivedDataKey << TEXT("Header");

				TArrayView<const uint8> HeaderView((uint8*)&Header, sizeof(FDDCChunkingHeader));

				GetDerivedDataCacheRef().Put(HeaderKey.ToString(), HeaderView, OwnerPathName);
			}

			for (int32 ChunkIndex = 0; ChunkIndex < NumChunks; ++ChunkIndex)
			{
				const int64 ChunkStart = ChunkIndex * ChunkSize;
				const uint64 BytesInChunk = FMath::Min(ChunkSize, DerivedData.Num() - ChunkStart);

				TArrayView<const uint8> ChunkData(DerivedData.GetData() + ChunkStart, (int32)BytesInChunk);

				TStringBuilder<512> ChunkKey;
				ChunkKey << DerivedDataKey << TEXT("Chunk") << ChunkIndex;
				GetDerivedDataCacheRef().Put(ChunkKey.ToString(), ChunkData, OwnerPathName);
			}
		}
	}
} //namespace DDCUtils64Bit

/** This code verify that the data is all in sync index buffer versus sections data. It is active only in debug build*/
void VerifyAllLodSkeletalMeshModelIntegrity(USkinnedAsset* Owner)
{
	if (!Owner || !Owner->GetImportedModel())
	{
		return;
	}

	FSkeletalMeshModel* SkelMeshModel = Owner->GetImportedModel();
	for (int32 LODIndex = 0; LODIndex < SkelMeshModel->LODModels.Num(); LODIndex++)
	{
		FSkeletalMeshLODModel* LODModel = &(SkelMeshModel->LODModels[LODIndex]);
		int32 SectionsVertexNum = 0;
		int32 SectionsTriangleNum = 0;
		for (const FSkelMeshSection& Section : LODModel->Sections)
		{
			if (Section.NumTriangles == 0)
			{
				continue; // don't validate index/vertex buffer for empty sections
			}

			SectionsVertexNum += Section.GetNumVertices();
			SectionsTriangleNum += Section.NumTriangles;
			int32 LastSectionIndexBuffer = Section.BaseIndex + (Section.NumTriangles * 3) - 1;
			if (LODModel->IndexBuffer.IsValidIndex(LastSectionIndexBuffer))
			{
				uint32 FirstSectionIndexBufferValue = LODModel->IndexBuffer[Section.BaseIndex];
				uint32 LastSectionIndexBufferValue = LODModel->IndexBuffer[LastSectionIndexBuffer];
				if (FirstSectionIndexBufferValue < Section.BaseVertexIndex || LastSectionIndexBufferValue >= Section.BaseVertexIndex + Section.GetNumVertices())
				{
					UE_ASSET_LOG(LogSkeletalMesh, Error, Owner, TEXT("The source model is corrupted! Section triangle refers to a vertex not in the section. LOD %d"), LODIndex);
				}
			}
			else
			{
				UE_ASSET_LOG(LogSkeletalMesh, Error, Owner, TEXT("The source model is corrupted! Section index buffer is invalid. LOD %d"), LODIndex);
			}
		}

		if (LODModel->NumVertices != SectionsVertexNum)
		{
			UE_ASSET_LOG(LogSkeletalMesh, Error, Owner, TEXT("The source model is corrupted! Total sections vertex count is different from source model vertex count. LOD %d"), LODIndex);
		}
		if ((LODModel->IndexBuffer.Num() / 3) != SectionsTriangleNum)
		{
			UE_ASSET_LOG(LogSkeletalMesh, Error, Owner, TEXT("The source model is corrupted! Total sections triangle count is different from source model triangle count (index count divide by 3). LOD %d"), LODIndex);
		}
	}
}

void FSkeletalMeshUnifiedBoneIndices::RemapVertex(TConstArrayView<uint16> BoneMap, FSoftSkinVertex& Vertex) const
{
	check(IsValid());

	for (int32 InfluenceIndex = 0; InfluenceIndex < MAX_TOTAL_INFLUENCES; ++InfluenceIndex)
	{
		const int32 LocalBoneIndex = Vertex.InfluenceBones[InfluenceIndex];
		if (LocalBoneIndex >= BoneMap.Num())
		{
			UE_LOG(LogSkeletalMesh, Error,
				TEXT("RemapVertex: LocalBoneIndex %d >= BoneMap.Num() %d (influence %d, weight %d). "
					 "Vertex references a BoneMap slot that doesn't exist — likely bone reduction "
					 "shrunk the BoneMap without remapping all vertex indices."),
				LocalBoneIndex, BoneMap.Num(), InfluenceIndex, (int32)Vertex.InfluenceWeights[InfluenceIndex]);
			Vertex.InfluenceBones[InfluenceIndex] = 0;
			Vertex.InfluenceWeights[InfluenceIndex] = 0;
			continue;
		}

		const int32 SkeletonBoneIndex = BoneMap[LocalBoneIndex];
		if (SkeletonBoneIndex >= SkeletonToUnifiedBoneIndex.Num())
		{
			UE_LOG(LogSkeletalMesh, Error,
				TEXT("RemapVertex: SkeletonBoneIndex %d >= SkeletonToUnifiedBoneIndex.Num() %d"),
				SkeletonBoneIndex, SkeletonToUnifiedBoneIndex.Num());
			Vertex.InfluenceBones[InfluenceIndex] = 0;
			Vertex.InfluenceWeights[InfluenceIndex] = 0;
			continue;
		}

		const uint16 UnifiedIndex = SkeletonToUnifiedBoneIndex[SkeletonBoneIndex];
		if (UnifiedIndex == 0xFFFF && Vertex.InfluenceWeights[InfluenceIndex] != 0)
		{
			UE_LOG(LogSkeletalMesh, Error,
				TEXT("RemapVertex: skeleton bone %d (from BoneMap[%d]) has no unified mapping "
					 "but vertex has non-zero weight %d — vertex references a bone not in the unified map."),
				SkeletonBoneIndex, LocalBoneIndex, (int32)Vertex.InfluenceWeights[InfluenceIndex]);
			Vertex.InfluenceBones[InfluenceIndex] = 0;
			Vertex.InfluenceWeights[InfluenceIndex] = 0;
			continue;
		}

		Vertex.InfluenceBones[InfluenceIndex] = UnifiedIndex;
	}
}

inline TArray<FBoneIndexType> BuildUnifiedBoneMap(USkinnedAsset* Owner)
{
	FSkeletalMeshModel* SkelMeshModel = Owner->GetImportedModel();
	check(SkelMeshModel);

	bool bOptimizeForInstancing = false;

	for (int32 LODIndex = 0; LODIndex < Owner->GetLODNum(); LODIndex++)
	{
		const FSkeletalMeshLODInfo* LODInfo = Owner->GetLODInfo(LODIndex);
		check(LODInfo);
		if (LODInfo->BuildSettings.bOptimizeForInstancing)
		{
			bOptimizeForInstancing = true;
			break;
		}
	}

	TArray<FBoneIndexType> UnifiedBoneMap;

	if (bOptimizeForInstancing)
	{
		USkeletalMesh* SkelMesh = Cast<USkeletalMesh>(Owner);

		IMeshUtilities& MeshUtilities = FModuleManager::Get().LoadModuleChecked<IMeshUtilities>("MeshUtilities");

		for (int32 LODIndex = 0; LODIndex < SkelMeshModel->LODModels.Num(); LODIndex++)
		{
			// Get the per-LOD reparent map from bone reduction. A bone can be removed from LOD N
			// but kept by LOD 0 — each LOD's reduction only applies to its own vertex data, so we
			// query per-LOD and only add reparent targets when processing that specific LOD.
			TMap<uint16, uint16> BonesToReplace;
			if (SkelMesh)
			{
				MeshUtilities.GetBoneReductionData(SkelMesh, LODIndex, BonesToReplace);
			}

			// Add a bone to the unified map, and if reduction would remove it on this LOD, also add
			// the reparent target (the bone that vertex weights get redirected to). Without this the
			// reparent target would be missing from the unified map if no MeshDescription vertex
			// directly referenced it, causing post-reduction Section.BoneMap entries to have no
			// corresponding unified slot.
			auto AddBoneAndReparentTarget = [&UnifiedBoneMap, &BonesToReplace](int32 BoneIndex)
			{
				if (BoneIndex == INDEX_NONE)
				{
					return;
				}
				UnifiedBoneMap.AddUnique((FBoneIndexType)BoneIndex);
				if (const uint16* ReparentTarget = BonesToReplace.Find((uint16)BoneIndex))
				{
					UnifiedBoneMap.AddUnique(*ReparentTarget);
				}
			};

			// For LODs that have a MeshDescription, read bone indices from it rather than
			// from the ImportedModel's section bone maps, which may be stale from a previous build.
			const FMeshDescription* MeshDescription = SkelMesh ? SkelMesh->GetMeshDescription(LODIndex) : nullptr;
			if (MeshDescription)
			{
				const FSkeletalMeshConstAttributes MeshAttributes(*MeshDescription);

				// Iterate all skin weight profiles, not just the default. Alternate skin weight
				// profiles can reference additional bones that the base weights don't, and those
				// bones end up in the post-reduction Section.BoneMap via the chunk builder
				// (see FSkeletalMeshTools "Add the alternate bones").
				TArray<FName> ProfileNames = MeshAttributes.GetSkinWeightProfileNames();
				ProfileNames.AddUnique(NAME_None); // Ensure the default profile is always included.

				for (const FName ProfileName : ProfileNames)
				{
					const FSkinWeightsVertexAttributesConstRef VertexSkinWeights = MeshAttributes.GetVertexSkinWeights(ProfileName);
					if (!VertexSkinWeights.IsValid())
					{
						continue;
					}

					for (const FVertexID VertexID : MeshDescription->Vertices().GetElementIDs())
					{
						const FVertexBoneWeightsConst BoneWeights = VertexSkinWeights.Get(VertexID);
						for (int32 InfluenceIndex = 0; InfluenceIndex < BoneWeights.Num(); ++InfluenceIndex)
						{
							if (BoneWeights[InfluenceIndex].GetWeight() > 0.0f)
							{
								AddBoneAndReparentTarget(BoneWeights[InfluenceIndex].GetBoneIndex());
							}
						}
					}
				}
			}
			else
			{
				// For LODs without a MeshDescription (e.g. generated/simplified LODs), fall back to
				// reading the existing section bone maps which are stable and won't be rebuilt.
				const FSkeletalMeshLODModel& LOD = SkelMeshModel->LODModels[LODIndex];
				for (const FSkelMeshSection& Section : LOD.Sections)
				{
					for (FBoneIndexType BoneIndex : Section.BoneMap)
					{
						AddBoneAndReparentTarget(BoneIndex);
					}
				}
			}

			// Also iterate alternate skin weight profiles' SourceModelInfluences. These are imported
			// via the older alternate-influence path (FLODUtilities::UpdateAlternateSkinWeights) and
			// stored on LODModel.SkinWeightProfiles separately from MeshDescription attributes. The
			// chunk builder reads these via AlternateBoneIDs and adds their bones to Section.BoneMap,
			// so we must include them in the unified map too.
			{
				const FSkeletalMeshLODModel& AltLOD = SkelMeshModel->LODModels[LODIndex];
				for (const TPair<FName, FImportedSkinWeightProfileData>& Pair : AltLOD.SkinWeightProfiles)
				{
					for (const SkeletalMeshImportData::FVertInfluence& Influence : Pair.Value.SourceModelInfluences)
					{
						if (Influence.Weight > 0.0f)
						{
							AddBoneAndReparentTarget(Influence.BoneIndex);
						}
					}
				}
			}

			// Always add the root bone for cases where content falls back to it.
			UnifiedBoneMap.Add(0);
		}

		UnifiedBoneMap.Sort();
		UnifiedBoneMap.SetNum(Algo::Unique(UnifiedBoneMap));
	}

	return UnifiedBoneMap;
}

FString FSkeletalMeshRenderData::GetDerivedDataKey(const ITargetPlatform* TargetPlatform, USkinnedAsset* Owner)
{
	return Owner->BuildDerivedDataKey(TargetPlatform);
}

void FSkeletalMeshRenderData::Cache(const ITargetPlatform* TargetPlatform, USkinnedAsset* Owner, FSkinnedAssetCompilationContext* ContextPtr)
{
	check(Owner);
	check(ContextPtr);

	check(LODRenderData.Num() == 0); // Should only be called on new, empty RenderData
	check(TargetPlatform);

	check(NaniteResourcesPtr.IsValid());
	Nanite::FResources& NaniteResources = *NaniteResourcesPtr.Get();

	if (USkeletalMesh* SkelMesh = Cast<USkeletalMesh>(Owner))
	{
		if (SkelMesh->IsNaniteAssembly() && !NaniteAssembliesSupported())
		{
			// Pop up a notification to warn the user about assembly support
			extern void ShowNaniteAssembliesDisabledWarningNotification();
			ShowNaniteAssembliesDisabledWarningNotification();
		}
	}

	auto SerializeLODModelDDCData = [&Owner](FSkeletalMeshLODModel* LODModel, FArchive& Ar)
	{
		//Make sure we add everything FSkeletalMeshLODModel got modified by the skeletal mesh builder
		Ar << LODModel->Sections;
		Ar << LODModel->NumVertices;
		Ar << LODModel->NumTexCoords;
		Ar << LODModel->IndexBuffer;
		Ar << LODModel->ActiveBoneIndices;
		Ar << LODModel->RequiredBones;
		Ar << LODModel->MeshToImportVertexMap;
		Ar << LODModel->MaxImportVertex;
		TArray<uint32>& RawPointIndices = LODModel->GetRawPointIndices();
		Ar << RawPointIndices;
	};

	{
		COOK_STAT(auto Timer = SkeletalMeshCookStats::UsageStats.TimeSyncWork());
		int32 T0 = FPlatformTime::Cycles();

		// When we import a skeletal mesh, in some cases the asset is not yet built, and the user section data and the inline cache are not set
		// until the initial build. This is due to the section count which is establish by the initial build of the import data. The section count
		// is part of the key because users can change section settings(see UserSectionData). So when we do a initial build we do not compute yet
		// the key and force the build code path, the key will be compute after the build and the DDC data will be store with the computed key.
		const bool bAllowDDCFetch = Owner->IsInitialBuildDone();
		if (bAllowDDCFetch)
		{
			DerivedDataKey = Owner->BuildDerivedDataKey(TargetPlatform);
		}
		
		// If we have an initial build, the ddc key will be computed only after the build. Some structure are missing until we first build the asset to get the drived data key
		
		TArray64<uint8> DerivedData;
		if (bAllowDDCFetch && DDCUtils64Bit::GetSynchronous(DerivedDataKey, Owner, DerivedData))
		{
			COOK_STAT(Timer.AddHit(DerivedData.Num()));

			FLargeMemoryReader Ar(DerivedData.GetData(), DerivedData.Num(), ELargeMemoryReaderFlags::Persistent);

			FSkeletalMeshModel* SkelMeshModel = Owner->GetImportedModel();
			check(SkelMeshModel);

			// Get the morph target data, we put it in the compilation context to apply them in the game thread before the InitResources
			if (Owner->GetMorphTargets().Num() > 0)
			{
				ContextPtr->FinishBuildMorphTargetData = Owner->GetMorphTargets()[0]->CreateFinishBuildMorphTargetData();
			}
			else
			{
				// Create and initialize the FinishBuildInternalData, use the class default object to call the virtual function
				ContextPtr->FinishBuildMorphTargetData = UMorphTarget::StaticClass()->GetDefaultObject<UMorphTarget>()->CreateFinishBuildMorphTargetData();
			}
			check(ContextPtr->FinishBuildMorphTargetData);
			ContextPtr->FinishBuildMorphTargetData->LoadFromMemoryArchive(Ar);

			// Serialize the LODModel sections since they are dependent on the reduction
			for (int32 LODIndex = 0; LODIndex < SkelMeshModel->LODModels.Num(); LODIndex++)
			{
				FSkeletalMeshLODModel* LODModel = &(SkelMeshModel->LODModels[LODIndex]);
				SerializeLODModelDDCData(LODModel, Ar);
				LODModel->SyncronizeUserSectionsDataArray();
			}

			Serialize(Ar, Owner);

			for (int32 LODIndex = 0; LODIndex < LODRenderData.Num(); ++LODIndex)
			{
				FSkeletalMeshLODRenderData& LODData = LODRenderData[LODIndex];
				if (LODData.bStreamedDataInlined)
				{
					break;
				}
				constexpr uint8 DummyStripFlags = 0;
				const bool bForceKeepCPUResources = FSkeletalMeshLODRenderData::ShouldForceKeepCPUResources();
				const bool bNeedsCPUAccess = FSkeletalMeshLODRenderData::ShouldKeepCPUResources(Owner, LODIndex, bForceKeepCPUResources);
				LODData.SerializeStreamedData(Ar, Owner, LODIndex, DummyStripFlags, bNeedsCPUAccess, bForceKeepCPUResources);
			}

			int32 T1 = FPlatformTime::Cycles();
			UE_LOGF(LogSkeletalMesh, Verbose, "Skeletal Mesh found in DDC [%fms] %ls", FPlatformTime::ToMilliseconds(T1 - T0), *Owner->GetPathName());
		}
		else
		{
			UE_LOGF(LogSkeletalMesh, Log, "Building Skeletal Mesh %ls...",*Owner->GetName());

			// Allocate empty entries for each LOD level in source mesh
			FSkeletalMeshModel* SkelMeshModel = Owner->GetImportedModel();
			check(SkelMeshModel);

			TArray<FBoneIndexType> UnifiedBoneMapStorage = BuildUnifiedBoneMap(Owner);

			// Assign to the temporary storage for BuildLODModel. This is reassigned after the loop to the last LOD's value which is permanent storage.
			UnifiedBoneMap = UnifiedBoneMapStorage;

			for (int32 LODIndex = 0; LODIndex < SkelMeshModel->LODModels.Num(); LODIndex++)
			{
				Owner->BuildLODModel(*this, TargetPlatform, LODIndex);

				const FSkeletalMeshLODModel* LODModel = &(SkelMeshModel->LODModels[LODIndex]);
				const FSkeletalMeshLODInfo* LODInfo = Owner->GetLODInfo(LODIndex);
				check(LODInfo);

				FSkeletalMeshLODRenderData* LODData = new FSkeletalMeshLODRenderData();
				LODRenderData.Add(LODData);
				
				// Get the UVs and tangents precision build settings flag specific for this LOD index
				ESkeletalMeshVertexFlags VertexBufferBuildFlags = Owner->GetVertexBufferFlags();
				{
					const bool bUseFullPrecisionUVs = LODInfo->BuildSettings.bUseFullPrecisionUVs;
					const bool bUseHighPrecisionTangentBasis = LODInfo->BuildSettings.bUseHighPrecisionTangentBasis;
					const bool bUseBackwardsCompatibleF16TruncUVs = LODInfo->BuildSettings.bUseBackwardsCompatibleF16TruncUVs;
					const bool bUseHighPrecisionWeights = LODInfo->BuildSettings.bUseHighPrecisionSkinWeights;
					if (bUseFullPrecisionUVs)
					{
						VertexBufferBuildFlags |= ESkeletalMeshVertexFlags::UseFullPrecisionUVs;
					}
					if (bUseHighPrecisionTangentBasis)
					{
						VertexBufferBuildFlags |= ESkeletalMeshVertexFlags::UseHighPrecisionTangentBasis;
					}
					if (bUseBackwardsCompatibleF16TruncUVs)
					{
						VertexBufferBuildFlags |= ESkeletalMeshVertexFlags::UseBackwardsCompatibleF16TruncUVs;
					}
					if (bUseHighPrecisionWeights)
					{
						VertexBufferBuildFlags |= ESkeletalMeshVertexFlags::UseHighPrecisionWeights;
					}
				}
				FSkeletalMeshLODRenderData::FBuildSettings BuildSettings;
				BuildSettings.UnifiedBoneMap = UnifiedBoneMap;
				BuildSettings.BuildFlags = VertexBufferBuildFlags;
				BuildSettings.bBuildHalfEdgeBuffers = Owner->HasHalfEdgeBuffer(LODIndex);

				LODData->BuildFromLODModel(LODModel, LODInfo->VertexAttributes, BuildSettings);
			}

			// Point the render data's unified bone map at the last LOD's storage.
			InitUnifiedBoneMapFromLODs();

			FLargeMemoryWriter Ar(0, /*bIsPersistent=*/ true);

			int32 MorphTargetNumber = Owner->GetMorphTargets().Num();
			Ar << MorphTargetNumber;
			for (int32 MorphTargetIndex = 0; MorphTargetIndex < MorphTargetNumber; ++MorphTargetIndex)
			{
				Owner->GetMorphTargets()[MorphTargetIndex]->SerializeMemoryArchive(Ar);
			}
			// No need to serialize the morph target mapping since we will rebuild the mapping when loading a ddc

			// Serialize the LODModel sections since they are dependent on the reduction
			for (int32 LODIndex = 0; LODIndex < SkelMeshModel->LODModels.Num(); LODIndex++)
			{
				FSkeletalMeshLODModel* LODModel = &(SkelMeshModel->LODModels[LODIndex]);
				SerializeLODModelDDCData(LODModel, Ar);
			}

			IMeshBuilderModule& MeshBuilderModule = IMeshBuilderModule::GetForPlatform(TargetPlatform);
			MeshBuilderModule.PostBuildSkeletalMesh(this, Owner);

			// Serialize the render data
			Serialize(Ar, Owner);

			for (int32 LODIndex = 0; LODIndex < LODRenderData.Num(); ++LODIndex)
			{
				FSkeletalMeshLODRenderData& LODData = LODRenderData[LODIndex];
				if (LODData.bStreamedDataInlined)
				{
					break;
				}
				const uint8 LODStripFlags = FSkeletalMeshLODRenderData::GenerateClassStripFlags(Ar, Owner, LODIndex);
				const bool bForceKeepCPUResources = FSkeletalMeshLODRenderData::ShouldForceKeepCPUResources();
				const bool bNeedsCPUAccess = FSkeletalMeshLODRenderData::ShouldKeepCPUResources(Owner, LODIndex, bForceKeepCPUResources);
				LODData.SerializeStreamedData(Ar, Owner, LODIndex, LODStripFlags, bNeedsCPUAccess, bForceKeepCPUResources);
			}

			//Recompute the derived data key in case there was some data correction during the build process, this make sure the DDC key is always representing the correct build result.
			//There should never be correction of the data during the build, the data has to be corrected in the post load before calling this function.
			FString BuiltDerivedDataKey = Owner->BuildDerivedDataKey(TargetPlatform);
			//Only compare keys if the ddc fetch was allowed
			if (bAllowDDCFetch)
			{
				if (BuiltDerivedDataKey != DerivedDataKey)
				{
					//If we are in this case we should resave the asset so the source data will be the same and we can use this DDC. Reduction can change the number of sections and the user section data is in the DDC key.
					//So if we change the reduction algorithm, its possible we fall in this situation.
					//We save the real data key which force the asset to always rebuild when the editor is loading it until the user save it
					UE_LOGF(LogSkeletalMesh, Log, "Skeletal mesh [%ls]: The derived data key is different after the build. Save the asset to avoid rebuilding it everytime the editor load it.", *Owner->GetPathName());
				}
			}
			else
			{
				//After the initial build we set the key to the built one
				DerivedDataKey = BuiltDerivedDataKey;
			}

			//Store the data using the built key to avoid DDC corruption
			TArrayView64<const uint8> ArView(Ar.GetData(), Ar.TotalSize());
			DDCUtils64Bit::Put(BuiltDerivedDataKey, Owner, ArView);

			int32 T1 = FPlatformTime::Cycles();
			UE_LOGF(LogSkeletalMesh, Log, "Built Skeletal Mesh [%.2fs] %ls", FPlatformTime::ToMilliseconds(T1 - T0) / 1000.0f, *Owner->GetPathName());
			COOK_STAT(Timer.AddMiss(DerivedData.Num()));
		}
	}
	VerifyAllLodSkeletalMeshModelIntegrity(Owner);
}

void FSkeletalMeshRenderData::SyncUVChannelData(const TArray<FSkeletalMaterial>& ObjectData)
{
	TUniquePtr< TArray<FMeshUVChannelInfo> > UpdateData = MakeUnique< TArray<FMeshUVChannelInfo> >();
	UpdateData->Empty(ObjectData.Num());

	for (const FSkeletalMaterial& SkeletalMaterial : ObjectData)
	{
		UpdateData->Add(SkeletalMaterial.UVChannelData);
	}

	// SyncUVChannelData can be called from any thread during async skeletal mesh compilation. 
	// There is currently multiple race conditions in ENQUEUE_RENDER_COMMAND making it unsafe to be called from
	// any other thread than rendering or game because of the render thread suspension mecanism.
	// We sidestep the issue here by avoiding a call to ENQUEUE_RENDER_COMMAND if the resource has not been initialized and is still unknown
	// to the render thread.
	if (bInitialized)
	{
		ENQUEUE_RENDER_COMMAND(SyncUVChannelData)(UE::RenderCommandPipe::SkeletalMesh, [this, UpdateData = MoveTemp(UpdateData)]
		{
			Swap(UVChannelDataPerMaterial, *UpdateData.Get());
		});
	}
	else
	{
		Swap(UVChannelDataPerMaterial, *UpdateData.Get());
	}
}

#endif // WITH_EDITOR

FSkeletalMeshRenderData::FSkeletalMeshRenderData()
	: bReadyForStreaming(false)
	, NumInlinedLODs(0)
	, NumNonOptionalLODs(0)
	, CurrentFirstLODIdx(0)
	, PendingFirstLODIdx(0)
	, LODBiasModifier(0)
	, bSupportRayTracing(true)
	, bInitialized(false)
{
	ClearNaniteResources(NaniteResourcesPtr);
}

FSkeletalMeshRenderData::~FSkeletalMeshRenderData()
{
	FRenderResourceNameScope FatalScope(OwnerName);
	
	FSkeletalMeshLODRenderData** LODRenderDataArray = LODRenderData.GetData();
	for (int32 LODIndex = 0; LODIndex < LODRenderData.Num(); ++LODIndex)
	{
		LODRenderDataArray[LODIndex]->Release();
		// Prevent the array from calling the destructor to handle correctly the refcount.
		// For compatibility reason, LODRenderDataArray is using ptr directly instead of TRefCountPtr.
		LODRenderDataArray[LODIndex] = nullptr;
	}
	LODRenderData.Empty();
}

int32 FSkeletalMeshRenderData::GetNumNonStreamingLODs() const
{
	int LODCount = 0;
	for (int32 Idx = LODRenderData.Num() - 1; Idx >= 0; --Idx)
	{
		if (LODRenderData[Idx].bStreamedDataInlined)
		{
			++LODCount;
		}
		else
		{
			break;
		}
	}

	if (LODCount == 0 && LODRenderData.Num())
	{
		return 1;
	}
	else
	{
		return LODCount;
	}
}

int32 FSkeletalMeshRenderData::GetNumNonOptionalLODs() const
{
	int LODCount = 0;
	for (int32 Idx = LODRenderData.Num() - 1; Idx >= 0; --Idx)
	{
		// Make sure GetNumNonOptionalLODs() is bigger than GetNumNonStreamingLODs().
		if (LODRenderData[Idx].bStreamedDataInlined || !LODRenderData[Idx].bIsLODOptional)
		{
			++LODCount;
		}
		else
		{
			break;
		}
	}

	if (LODCount == 0 && LODRenderData.Num())
	{
		return 1;
	}
	else
	{
		return LODCount;
	}
}

void FSkeletalMeshRenderData::Serialize(FArchive& Ar, USkinnedAsset* Owner)
{
	DECLARE_SCOPE_CYCLE_COUNTER(TEXT("FSkeletalMeshRenderData::Serialize"), STAT_SkeletalMeshRenderData_Serialize, STATGROUP_LoadTime);
	Ar.UsingCustomVersion(FSkeletalMeshCustomVersion::GUID);

#if PLATFORM_DESKTOP

	if (Ar.IsFilterEditorOnly())
	{
		int32 MinMobileLODIdx = 0;
		bool bShouldSerialize = GSkeletalMeshKeepMobileMinLODSettingOnDesktop != 0;
#if WITH_EDITOR
		if (Ar.IsSaving())
		{
			if (Ar.CookingTarget()->GetPlatformInfo().PlatformGroupName == TEXT("Desktop")
				&& GStripSkeletalMeshLodsDuringCooking != 0
				&& GSkeletalMeshKeepMobileMinLODSettingOnDesktop != 0)
			{
				// Serialize 0 value when per quality level properties are used
				if (!Owner->IsMinLodQualityLevelEnable())
				{
					MinMobileLODIdx = Owner->GetMinLod().GetValueForPlatform(TEXT("Mobile")) - Owner->GetMinLod().GetValueForPlatform(TEXT("Desktop"));
					// Will be cast to uint8 when applying LOD bias. Also, make sure it's not < 0,
					// which can happen if the desktop min LOD is higher than the mobile setting
					MinMobileLODIdx = FMath::Clamp(MinMobileLODIdx, 0, 255); 
				}													
			}
			else
			{
				bShouldSerialize = false;
			}
		}
#endif

		if (bShouldSerialize)
		{
			Ar << MinMobileLODIdx;

			if (Ar.IsLoading() && GMaxRHIFeatureLevel == ERHIFeatureLevel::ES3_1)
			{
				LODBiasModifier = MinMobileLODIdx;
			}
		}
	}
#endif // PLATFORM_DESKTOP

	LODRenderData.Serialize(Ar, Owner);

	if (Ar.IsLoading())
	{
		InitUnifiedBoneMapFromLODs();
	}

	check(NaniteResourcesPtr.IsValid());
	NaniteResourcesPtr->Serialize(Ar, Owner, false /* bCooked */);

#if WITH_EDITOR
	if (Ar.IsSaving())
	{
		NumInlinedLODs = GetNumNonStreamingLODs();
		NumNonOptionalLODs = GetNumNonOptionalLODs();
	}
#endif
	Ar << NumInlinedLODs << NumNonOptionalLODs;
#if WITH_EDITOR
	//Recompute on load because previously we were storing NumOptionalLODs, which is less convenient because it includes first LODs (and can be stripped by MinMip).
	if (Ar.IsLoading())
	{
		NumInlinedLODs = GetNumNonStreamingLODs();
		NumNonOptionalLODs = GetNumNonOptionalLODs();
	}
#endif
	ensure(LODRenderData.Num() >= NumInlinedLODs);

#if WITH_EDITORONLY_DATA
	const bool bUsingCookedEditorData = Owner->GetOutermost()->bIsCookedForEditor;
	if (bUsingCookedEditorData && Ar.IsLoading())
	{
		CurrentFirstLODIdx = Owner->GetMinLodIdx();
	}
	else
#endif
	{
		CurrentFirstLODIdx = LODRenderData.Num() - NumInlinedLODs;
	}

	PendingFirstLODIdx = CurrentFirstLODIdx;

PRAGMA_DISABLE_DEPRECATION_WARNINGS
	bSupportRayTracing = Owner->GetSupportRayTracing();
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

void FSkeletalMeshRenderData::InitResources(bool bNeedsVertexColors, USkinnedAsset* Owner)
{
	if (!bInitialized)
	{
#if RHI_RAYTRACING
		if (IsRayTracingAllowed())
		{
			ENQUEUE_RENDER_COMMAND(SkeletalMeshInitRayTracingGeometryGroup)(UE::RenderCommandPipe::SkeletalMesh,
				[this, NumLODs = LODRenderData.Num()]
				{
					check(RayTracingGeometryGroupHandle.IsNull());
					RayTracingGeometryGroupHandle = GRayTracingGeometryManager->RegisterRayTracingGeometryGroup(NumLODs, CurrentFirstLODIdx);
				});
		}
#endif

		// initialize resources for each lod
		for (int32 LODIndex = 0; LODIndex < LODRenderData.Num(); LODIndex++)
		{
			FSkeletalMeshLODRenderData& RenderData = LODRenderData[LODIndex];

			if(RenderData.GetNumVertices() > 0)
			{
				RenderData.InitResources(bNeedsVertexColors, LODIndex, Owner);
			}
		}

		check(NaniteResourcesPtr.IsValid());
		NaniteResourcesPtr->InitResources(Owner);

		ENQUEUE_RENDER_COMMAND(CmdSetSkeletalMeshReadyForStreaming)(UE::RenderCommandPipe::SkeletalMesh,
			[this, Owner]
		{
			bReadyForStreaming = true;
		});

		bInitialized = true;
	}
}

void FSkeletalMeshRenderData::ReleaseResources()
{
	if (bInitialized)
	{
		// release resources for each lod
		for (int32 LODIndex = 0; LODIndex < LODRenderData.Num(); LODIndex++)
		{
			LODRenderData[LODIndex].ReleaseResources();
		}

#if RHI_RAYTRACING
		if (IsRayTracingAllowed())
		{
			ENQUEUE_RENDER_COMMAND(SkeletalMeshReleaseRayTracingGeometryGroup)(UE::RenderCommandPipe::SkeletalMesh,
				[this]
				{
					GRayTracingGeometryManager->ReleaseRayTracingGeometryGroup(RayTracingGeometryGroupHandle);
					RayTracingGeometryGroupHandle = {};
				});
		}
#endif

		check(NaniteResourcesPtr.IsValid());
		NaniteResourcesPtr->ReleaseResources();

		bInitialized = false;
	}
}

void FSkeletalMeshRenderData::InitStaticRayTracingGeometry(int32 LODIndex)
{
#if RHI_RAYTRACING
	FSkeletalMeshLODRenderData& LODData = LODRenderData[LODIndex];

	if (LODData.NumReferencingStaticSkeletalMeshObjects == 0)
	{
		FPositionVertexBuffer* PositionVertexBufferPtr = &LODData.StaticVertexBuffers.PositionVertexBuffer;
		FRawStaticIndexBuffer16or32Interface* IndexBufferPtr = LODData.MultiSizeIndexContainer.GetIndexBuffer();

		uint32 TrianglesCount = 0;
		for (int32 SectionIndex = 0; SectionIndex < LODData.RenderSections.Num(); SectionIndex++)
		{
			const FSkelMeshRenderSection& Section = LODData.RenderSections[SectionIndex];
			TrianglesCount += Section.NumTriangles;
		}

		TArray<FSkelMeshRenderSection>* RenderSections = &LODData.RenderSections;
		ENQUEUE_RENDER_COMMAND(InitSkeletalRenderStaticRayTracingGeometry)(UE::RenderCommandPipe::SkeletalMesh,
			[PositionVertexBufferPtr, IndexBufferPtr, TrianglesCount, RenderSections,
			LODIndex = LODIndex,
			SkelMeshRenderData = this,
			&RayTracingGeometry = LODData.StaticRayTracingGeometry,
			&SourceRayTracingGeometry = LODData.SourceRayTracingGeometry,
			&bReferencedByStaticSkeletalMeshObjects_RenderThread = LODData.bReferencedByStaticSkeletalMeshObjects_RenderThread](FRHICommandList& RHICmdList)
			{
				FRayTracingGeometryInitializer Initializer;
				Initializer.IndexBuffer = IndexBufferPtr->IndexBufferRHI;
				Initializer.TotalPrimitiveCount = TrianglesCount;
				Initializer.GeometryType = RTGT_Triangles;
				Initializer.bFastBuild = false;

				TArray<FRayTracingGeometrySegment> GeometrySections;
				GeometrySections.Reserve(RenderSections->Num());

				for (const FSkelMeshRenderSection& Section : *RenderSections)
				{
					FRayTracingGeometrySegment Segment;
					Segment.VertexBuffer = PositionVertexBufferPtr->VertexBufferRHI;
					Segment.VertexBufferElementType = VET_Float3;
					Segment.VertexBufferOffset = 0;
					Segment.VertexBufferStride = PositionVertexBufferPtr->GetStride();
					Segment.MaxVertices = PositionVertexBufferPtr->GetNumVertices();
					Segment.FirstPrimitive = Section.BaseIndex / 3;
					Segment.NumPrimitives = Section.NumTriangles;
					Segment.bEnabled = !Section.bDisabled && Section.bVisibleInRayTracing;
					GeometrySections.Add(Segment);
				}
				Initializer.Segments = GeometrySections;

				if (LODIndex < SkelMeshRenderData->CurrentFirstLODIdx) // According to GetMeshElementsConditionallySelectable(), non-resident LODs should just be skipped
				{
					Initializer.Type = ERayTracingGeometryInitializerType::StreamingDestination;
				}

				Initializer.SourceGeometry = SourceRayTracingGeometry.GetRHI();

				RayTracingGeometry.GroupHandle = SkelMeshRenderData->RayTracingGeometryGroupHandle;
				RayTracingGeometry.LODIndex = LODIndex;

				static const FName StaticRayTracingGeometryName("FSkeletalMeshLODRenderData.StaticRayTracingGeometry");
				static int32 DebugNumber = 0;
				Initializer.DebugName = FDebugName(StaticRayTracingGeometryName, DebugNumber++);
				Initializer.OwnerName = SkelMeshRenderData->OwnerName;

				RayTracingGeometry.SetInitializer(Initializer);

				RayTracingGeometry.SetResourceName(StaticRayTracingGeometryName);
				RayTracingGeometry.SetOwnerName(SkelMeshRenderData->OwnerName);

				RayTracingGeometry.InitResource(RHICmdList);

				if (LODIndex >= SkelMeshRenderData->CurrentFirstLODIdx && LODIndex < SkelMeshRenderData->LODRenderData.Num() - SkelMeshRenderData->NumInlinedLODs)
				{
					// If this LOD is not inlined and is currently streamed-in tag it as such
					RayTracingGeometry.SetAsStreamedIn();
				}

				bReferencedByStaticSkeletalMeshObjects_RenderThread = true;
			}
		);
	}

	LODData.NumReferencingStaticSkeletalMeshObjects++;
#endif // RHI_RAYTRACING
}

void FSkeletalMeshRenderData::ReleaseStaticRayTracingGeometry(int32 LODIndex)
{
#if RHI_RAYTRACING
	FSkeletalMeshLODRenderData& LODData = LODRenderData[LODIndex];

	if (LODData.NumReferencingStaticSkeletalMeshObjects > 0)
	{
		LODData.NumReferencingStaticSkeletalMeshObjects--;

		if (LODData.NumReferencingStaticSkeletalMeshObjects == 0)
		{
			ENQUEUE_RENDER_COMMAND(ResetStaticRayTracingGeometryFlag)(UE::RenderCommandPipe::SkeletalMesh,
				[&bReferencedByStaticSkeletalMeshObjects_RenderThread = LODData.bReferencedByStaticSkeletalMeshObjects_RenderThread]
				{
					bReferencedByStaticSkeletalMeshObjects_RenderThread = false;
				});

			BeginReleaseResource(&LODData.StaticRayTracingGeometry, &UE::RenderCommandPipe::SkeletalMesh);
		}
	}
#endif // RHI_RAYTRACING
}

uint32 FSkeletalMeshRenderData::GetNumBoneInfluences(int32 MinLODIndex) const
{
	uint32 NumBoneInfluences = 0;
	for (int32 LODIndex = MinLODIndex; LODIndex < LODRenderData.Num(); ++LODIndex)
	{
		const FSkeletalMeshLODRenderData& Data = LODRenderData[LODIndex];
		NumBoneInfluences = FMath::Max(NumBoneInfluences, Data.GetVertexBufferMaxBoneInfluences());
	}

	return NumBoneInfluences;
}

uint32 FSkeletalMeshRenderData::GetNumBoneInfluences() const
{
	return GetNumBoneInfluences(0);
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
bool FSkeletalMeshRenderData::RequiresCPUSkinning(ERHIFeatureLevel::Type FeatureLevel, int32 MinLODIndex) const
{
	// All platforms support 16-bit bone indices. A section's bone map cannot exceed 65536,
	// so GPU skinning is always viable. CPU skinning fallback is no longer needed for bone limits.
	return false;
}

bool FSkeletalMeshRenderData::RequiresCPUSkinning(ERHIFeatureLevel::Type FeatureLevel) const
{
	return RequiresCPUSkinning(FeatureLevel, 0);
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

void FSkeletalMeshRenderData::GetResourceSizeEx(FResourceSizeEx& CumulativeResourceSize)
{
	for (int32 LODIndex = 0; LODIndex < LODRenderData.Num(); ++LODIndex)
	{
		const FSkeletalMeshLODRenderData& RenderData = LODRenderData[LODIndex];
		RenderData.GetResourceSizeEx(CumulativeResourceSize);
	}

	GetNaniteResourcesSizeEx(NaniteResourcesPtr, CumulativeResourceSize);
}

SIZE_T FSkeletalMeshRenderData::GetCPUAccessMemoryOverhead() const
{
	SIZE_T Result = 0;
	for (int32 LODIndex = 0; LODIndex < LODRenderData.Num(); ++LODIndex)
	{
		const FSkeletalMeshLODRenderData& RenderData = LODRenderData[LODIndex];
		Result += RenderData.GetCPUAccessMemoryOverhead();
	}
	return Result;
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
int32 FSkeletalMeshRenderData::GetMaxBonesPerSection(int32 MinLODIdx) const
{
	int32 MaxBonesPerSection = 0;
	for (int32 LODIndex = MinLODIdx; LODIndex < LODRenderData.Num(); ++LODIndex)
	{
		const FSkeletalMeshLODRenderData& RenderData = LODRenderData[LODIndex];
		for (int32 SectionIndex = 0; SectionIndex < RenderData.RenderSections.Num(); ++SectionIndex)
		{
			MaxBonesPerSection = FMath::Max<int32>(MaxBonesPerSection, RenderData.RenderSections[SectionIndex].BoneMap.Num());
		}
	}
	return MaxBonesPerSection;
}

bool FSkeletalMeshRenderData::AnyRenderSectionCastsShadows(int32 MinLODIdx) const
{
	for (int32 LODIndex = MinLODIdx; LODIndex < LODRenderData.Num(); LODIndex++)
	{
		for (const FSkelMeshRenderSection& RenderSection : LODRenderData[LODIndex].RenderSections)
		{
			if (RenderSection.bCastShadow)
			{
				return true;
			}
		}
	}

	return false;
}

int32 FSkeletalMeshRenderData::GetMaxBonesPerSection() const
{
	return GetMaxBonesPerSection(0);
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

int32 FSkeletalMeshRenderData::GetFirstValidLODIdx(int32 MinIdx) const
{
	const int32 LODCount = LODRenderData.Num();
	if (LODCount == 0)
	{
		return INDEX_NONE;
	}

	int32 LODIndex = FMath::Clamp<int32>(MinIdx, 0, LODCount - 1);
	while (LODIndex < LODCount && !LODRenderData[LODIndex].GetNumVertices())
	{
		++LODIndex;
	}
	return (LODIndex < LODCount) ? LODIndex : INDEX_NONE;
}

bool FSkeletalMeshRenderData::HasValidNaniteData() const
{
	return NaniteResourcesPtr->PageStreamingStates.Num() > 0;
}

bool FSkeletalMeshRenderData::IsLODDataReady() const
{
	bool bIsReady = true;
	for (const FSkeletalMeshLODRenderData& LODData : LODRenderData)
	{
		bIsReady = bIsReady && LODData.IsDataReady();
	}

	return bIsReady;
}
