// Copyright Epic Games, Inc. All Rights Reserved.

#include "Engine/SkeletalMeshSampling.h"
#include "Engine/SkeletalMesh.h"
#include "Engine/SkinnedAssetCommon.h"
#include "Rendering/SkeletalMeshRenderData.h"
#include "UObject/NiagaraObjectVersion.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SkeletalMeshSampling)

//////////////////////////////////////////////////////////////////////////
//FSkeletalMeshAreaWeightedTriangleSampler

FSkeletalMeshAreaWeightedTriangleSampler::FSkeletalMeshAreaWeightedTriangleSampler()
	: Owner(nullptr)
	, TriangleIndices(nullptr)
	, LODIndex(INDEX_NONE)
{

}

void FSkeletalMeshAreaWeightedTriangleSampler::Init(USkeletalMesh* InOwner, int32 InLODIndex, TArray<int32>* InTriangleIndices)
{
	Owner = InOwner;
	check(InOwner);
	check(InOwner->IsValidLODIndex(InLODIndex));

	LODIndex = InLODIndex;
	TriangleIndices = InTriangleIndices;
	Initialize();
}

float FSkeletalMeshAreaWeightedTriangleSampler::GetWeights(TArray<float>& OutWeights)
{
	//If these hit, you're trying to get weights on a sampler that's not been initialized.
	check(Owner);
		
	FSkeletalMeshLODRenderData& LODData = Owner->GetResourceForRendering()->LODRenderData[LODIndex];
	FSkinWeightVertexBuffer* SkinWeightBuffer = &LODData.SkinWeightVertexBuffer;
	FRawStaticIndexBuffer16or32Interface* IndexBuffer = LODData.MultiSizeIndexContainer.GetIndexBuffer();

	auto GetTriVerts = [&](int32 Tri, FVector& OutPos0, FVector& OutPos1, FVector& OutPos2)
	{
		int32 Idx0 = IndexBuffer->Get(Tri);
		int32 Idx1 = IndexBuffer->Get(Tri + 1);
		int32 Idx2 = IndexBuffer->Get(Tri + 2);

		OutPos0 = GetSkeletalMeshRefVertLocation(Owner, LODData, *SkinWeightBuffer, Idx0);
		OutPos1 = GetSkeletalMeshRefVertLocation(Owner, LODData, *SkinWeightBuffer, Idx1);
		OutPos2 = GetSkeletalMeshRefVertLocation(Owner, LODData, *SkinWeightBuffer, Idx2);
	};

	const FSkeletalMeshSamplingInfo& SamplingInfo = Owner->GetSamplingInfo();
	
	float Total = 0.0f;
	if (TriangleIndices)
	{
		TArray<int32>& Tris = *TriangleIndices;
		int32 NumTris = Tris.Num();
		if (NumTris == 0)
		{
			return 0.0f;
		}

		OutWeights.Empty(NumTris);
		for (int32 i = 0; i < NumTris; ++i)
		{
			int32 TriIdx = Tris[i];
			FVector V0;
			FVector V1;
			FVector V2;
			GetTriVerts(TriIdx, V0, V1, V2);

			float Area = ((V1 - V0) ^ (V2 - V0)).Size() * 0.5f;
			OutWeights.Add(Area);
			Total += Area;
		}
	}
	else
	{
		//No explicit tri index buffer provided so use whole mesh.
		int32 NumIndices = LODData.MultiSizeIndexContainer.GetIndexBuffer()->Num();
		int32 NumTris = NumIndices / 3;
		OutWeights.Empty(NumTris);
		for (int32 TriIdx = 0; TriIdx < NumIndices; TriIdx += 3)
		{
			FVector V0;
			FVector V1;
			FVector V2;
			GetTriVerts(TriIdx, V0, V1, V2);

			float Area = ((V1 - V0) ^ (V2 - V0)).Size() * 0.5f;
			OutWeights.Add(Area);
			Total += Area;
		}
	}
	return Total;
}

//////////////////////////////////////////////////////////////////////////
//FSkeletalMeshSamplingWholeMeshBuiltData

bool FSkeletalMeshSamplingLODBuiltData::Serialize(FArchive& Ar)
{
	AreaWeightedTriangleSampler.Serialize(Ar);
	return true;
}

//////////////////////////////////////////////////////////////////////////
//FSkeletalMeshSamplingRegionBuiltData

bool FSkeletalMeshSamplingRegionBuiltData::Serialize(FArchive& Ar)
{
	Ar.UsingCustomVersion(FNiagaraObjectVersion::GUID);

	Ar << TriangleIndices;
	Ar << BoneIndices;
	AreaWeightedSampler.Serialize(Ar);

	if (Ar.CustomVer(FNiagaraObjectVersion::GUID) >= FNiagaraObjectVersion::SkeletalMeshVertexSampling)
	{
		Ar << Vertices;
	}

	return true;
}

//////////////////////////////////////////////////////////////////////////

FSkeletalMeshSamplingRegionMaterialFilter::FSkeletalMeshSamplingRegionMaterialFilter()
	: MaterialName(NAME_None)
{}

FSkeletalMeshSamplingRegionBoneFilter::FSkeletalMeshSamplingRegionBoneFilter()
	: BoneName(NAME_None)
	, bIncludeOrExclude(true)
	, bApplyToChildren(true)
{
}

//////////////////////////////////////////////////////////////////////////
//FSkeletalMeshSamplingRegion

FSkeletalMeshSamplingRegion::FSkeletalMeshSamplingRegion()
	: Name(NAME_None)
	, LODIndex(INDEX_NONE)
	, bSupportUniformlyDistributedSampling(false)
{

}

void FSkeletalMeshSamplingRegion::GetFilteredBones(FReferenceSkeleton& RefSkel, TArray<int32>& OutIncludedBoneIndices, TArray<int32>& OutExcludedBoneIndices)
{
	for (int32 RefBoneIdx = 0; RefBoneIdx < RefSkel.GetNum(); ++RefBoneIdx)
	{
		const FMeshBoneInfo& BoneInfo = RefSkel.GetRefBoneInfo()[RefBoneIdx];

		if (BoneFilters.Num() == 0)
		{
			OutIncludedBoneIndices.Add(RefBoneIdx);
		}

		for (FSkeletalMeshSamplingRegionBoneFilter& BoneFilter : BoneFilters)
		{
			bool bAddBone = false;
			int32 BoneIdx = RefSkel.FindBoneIndex(BoneFilter.BoneName);
			if (BoneIdx != INDEX_NONE)
			{
				if (BoneFilter.bApplyToChildren)
				{
					if (RefSkel.GetDepthBetweenBones(RefBoneIdx, BoneIdx) != INDEX_NONE)
					{
						if (BoneFilter.bIncludeOrExclude)
						{
							OutIncludedBoneIndices.Add(RefBoneIdx);
						}
						else
						{
							OutExcludedBoneIndices.Add(RefBoneIdx);
						}
					}
				}
				else
				{
					if (BoneIdx == RefBoneIdx)
					{
						if (BoneFilter.bIncludeOrExclude)
						{
							OutIncludedBoneIndices.Add(RefBoneIdx);
						}
						else
						{
							OutExcludedBoneIndices.Add(RefBoneIdx);
						}
					}
				}
			}
		}
	}
}

bool FSkeletalMeshSamplingRegion::IsMaterialAllowed(FName MaterialName)
{
	if (MaterialFilters.Num() == 0)
	{
		return true;
	}

	for (FSkeletalMeshSamplingRegionMaterialFilter& MaterialFilter : MaterialFilters)
	{
		if (MaterialFilter.MaterialName == MaterialName)
		{
			return true;
		}
	}

	return false;
}

//////////////////////////////////////////////////////////////////////////
//FSkeletalMeshSamplingInfo

#if WITH_EDITORONLY_DATA

FORCEINLINE_DEBUGGABLE void GetRegionValidData(USkeletalMesh* SkeletalMesh, FSkeletalMeshSamplingRegion& SamplingRegion,
	FSkeletalMeshLODRenderData& LODData, FReferenceSkeleton& RefSkel, FSkinWeightVertexBuffer* SkinWeightBuffer,
	TArray<int32>& IncludedBoneIndices, TArray<int32>& ExcludedBoneIndices, TArray<int32>& OutValidTris, TArray<int32>& OutValidVerts)
{
	int32 MaxBoneInfluences = SkinWeightBuffer->GetMaxBoneInfluences();
	auto VertIsValid = [&](FSkelMeshRenderSection& Section, int32 VertexIdx)
	{
#if !PLATFORM_LITTLE_ENDIAN
		// uint8[] elements in LOD.VertexBufferGPUSkin have been swapped for VET_UBYTE4 vertex stream use
		for (int32 InfluenceIndex = MAX_INFLUENCES - 1; InfluenceIndex >= MAX_INFLUENCES - Section.MaxBoneInfluences; InfluenceIndex--)
#else
		for (int32 InfluenceIndex = 0; InfluenceIndex < MaxBoneInfluences; InfluenceIndex++)
#endif
		{
			int32 BoneIndex = Section.BoneMap[SkinWeightBuffer->GetBoneIndex(VertexIdx, InfluenceIndex)];

			if (IncludedBoneIndices.Contains(BoneIndex) && !ExcludedBoneIndices.Contains(BoneIndex))
			{
				return true;
			}
		}

		return false;
	};
	
	for (FSkelMeshRenderSection& Section : LODData.RenderSections)
	{
		FName MaterialName = SkeletalMesh->GetMaterials()[Section.MaterialIndex].MaterialSlotName;

		if (SamplingRegion.IsMaterialAllowed(MaterialName))
		{
			int32 StartTri = Section.BaseIndex;
			int32 FinalIndex = StartTri + Section.NumTriangles * 3;
			for (int32 TriBase = StartTri; TriBase < FinalIndex; TriBase += 3)
			{
				int32 Idx0 = LODData.MultiSizeIndexContainer.GetIndexBuffer()->Get(TriBase);
				int32 Idx1 = LODData.MultiSizeIndexContainer.GetIndexBuffer()->Get(TriBase + 1);
				int32 Idx2 = LODData.MultiSizeIndexContainer.GetIndexBuffer()->Get(TriBase + 2);

				if (VertIsValid(Section, Idx0) || VertIsValid(Section, Idx1) || VertIsValid(Section, Idx2))
				{
					OutValidTris.Add(TriBase);
				}
			}

			int32 FirstVert = Section.BaseVertexIndex;
			int32 MaxVert = FirstVert + Section.GetNumVertices();
			for (int32 VertexIdx = Section.BaseVertexIndex ; VertexIdx < MaxVert; ++VertexIdx)
			{
				if (VertIsValid(Section, VertexIdx))
				{
					OutValidVerts.Add(VertexIdx);
				}
			}
		}
	}
}

void FSkeletalMeshSamplingInfo::BuildWholeMesh(USkeletalMesh* SkeletalMesh)
{
	//Build data for the whole mesh at each LOD.
	BuiltData.WholeMeshBuiltData.Reset(SkeletalMesh->GetLODNum());
	BuiltData.WholeMeshBuiltData.SetNum(SkeletalMesh->GetLODNum());
	for (int32 LODIndex = 0; LODIndex < SkeletalMesh->GetLODNum(); ++LODIndex)
	{
		FSkeletalMeshLODInfo& LODInfo = *SkeletalMesh->GetLODInfo(LODIndex);
		FSkeletalMeshSamplingLODBuiltData& WholeMeshBuiltData = BuiltData.WholeMeshBuiltData[LODIndex];

		if (LODInfo.bSupportUniformlyDistributedSampling)
		{
			WholeMeshBuiltData.AreaWeightedTriangleSampler.Init(SkeletalMesh, LODIndex, nullptr);
		}
	}
}

void FSkeletalMeshSamplingInfo::BuildRegions(USkeletalMesh* SkeletalMesh)
{	
	//Build the per region data.
	BuiltData.RegionBuiltData.Reset(Regions.Num());
	BuiltData.RegionBuiltData.SetNum(Regions.Num());
	for (int32 RegionIndex = 0; RegionIndex < Regions.Num(); ++RegionIndex)
	{
		FSkeletalMeshSamplingRegion& Region = Regions[RegionIndex];
		FSkeletalMeshSamplingRegionBuiltData& RegionBuiltData = BuiltData.RegionBuiltData[RegionIndex];
		int32 LODIndex = Region.LODIndex;
		if (LODIndex == INDEX_NONE)
		{
			LODIndex = SkeletalMesh->GetResourceForRendering()->LODRenderData.Num() - 1;
		}
		if (!SkeletalMesh->IsValidLODIndex(LODIndex))
		{
			continue;
		}

		FSkeletalMeshLODRenderData& LODData = SkeletalMesh->GetResourceForRendering()->LODRenderData[LODIndex];
		FSkinWeightVertexBuffer* SkinWeightBuffer = &LODData.SkinWeightVertexBuffer;
		check(SkinWeightBuffer);

		FReferenceSkeleton& RefSkel = SkeletalMesh->GetRefSkeleton();
		TArray<int32> IncludedBoneIndices;
		TArray<int32> ExcludedBoneIndices;
		Region.GetFilteredBones(RefSkel, IncludedBoneIndices, ExcludedBoneIndices);

		RegionBuiltData.BoneIndices = IncludedBoneIndices;

		GetRegionValidData(SkeletalMesh, Region, LODData, RefSkel, SkinWeightBuffer, IncludedBoneIndices, ExcludedBoneIndices, RegionBuiltData.TriangleIndices, RegionBuiltData.Vertices);

		if (Region.bSupportUniformlyDistributedSampling)
		{
			RegionBuiltData.AreaWeightedSampler.Init(SkeletalMesh, LODIndex, &RegionBuiltData.TriangleIndices);
		}
	}
}
#endif

bool FSkeletalMeshSamplingInfo::IsSamplingEnabled(const USkeletalMesh* OwnerMesh, int32 LODIndex)const
{
	check(OwnerMesh && OwnerMesh->IsValidLODIndex(LODIndex));
	if (OwnerMesh->GetLODInfo(LODIndex)->bAllowCPUAccess)
	{
		return true;
	}

	for (const FSkeletalMeshSamplingRegion& Region : Regions)
	{
		if (Region.LODIndex == LODIndex)
		{
			return true;
		}
	}

	return false;
}

