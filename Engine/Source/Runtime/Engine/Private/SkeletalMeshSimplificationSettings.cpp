// Copyright Epic Games, Inc. All Rights Reserved.

#include "Engine/SkeletalMeshSimplificationSettings.h"
#include "HAL/IConsoleManager.h"
#include "CoreGlobals.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SkeletalMeshSimplificationSettings)

static FAutoConsoleVariable CVarSkeletalMeshReductionModule(
	TEXT("r.SkeletalMeshReductionModule"),
	TEXT("SkeletalMeshReduction"),
	TEXT("Name of what skeletal mesh reduction module to choose. If blank it chooses any that exist.\n"),
	ECVF_ReadOnly);

USkeletalMeshSimplificationSettings::USkeletalMeshSimplificationSettings(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

FName USkeletalMeshSimplificationSettings::GetContainerName() const
{
	static const FName ContainerName("Project");
	return ContainerName;
}

FName USkeletalMeshSimplificationSettings::GetCategoryName() const
{
	static const FName EditorCategoryName("Editor");
	return EditorCategoryName;
}

void USkeletalMeshSimplificationSettings::SetSkeletalMeshReductionModuleName(FName InSkeletalMeshReductionModuleName)
{
	SkeletalMeshReductionModuleName = InSkeletalMeshReductionModuleName;
	CVarSkeletalMeshReductionModule->Set(*SkeletalMeshReductionModuleName.ToString());
}

void USkeletalMeshSimplificationSettings::PostInitProperties()
{
	Super::PostInitProperties();

#if WITH_EDITOR
	if (IsTemplate())
	{
		ImportConsoleVariableValues();
	}
#endif
}

#if WITH_EDITOR


void USkeletalMeshSimplificationSettings::PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	if (PropertyChangedEvent.Property)
	{
		ExportValuesToConsoleVariables(PropertyChangedEvent.Property);
	}
}


#endif

