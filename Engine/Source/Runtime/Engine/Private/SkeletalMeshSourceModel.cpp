// Copyright Epic Games, Inc. All Rights Reserved.

#include "Engine/SkeletalMeshSourceModel.h"

#include "Async/ParallelFor.h"
#include "Rendering/SkeletalMeshModel.h"

#if WITH_EDITOR
#include "Engine/SkeletalMesh.h"
#include "Engine/SkinnedAssetCommon.h"
#include "SkeletalMeshDescription.h"
#include "Rendering/SkeletalMeshLODImporterData.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(SkeletalMeshSourceModel)

USkeletalMeshDescriptionBulkData::USkeletalMeshDescriptionBulkData()
{
#if WITH_EDITOR
	// Create the skeletal mesh description template that will be used to create all FMeshDescription objects with
	// the correct attributes registered. 
	constexpr bool bTransient = true;
	PreallocatedMeshDescription = CreateDefaultSubobject<USkeletalMeshDescription>(TEXT("MeshDescription"), bTransient);
#endif
}


void USkeletalMeshDescriptionBulkData::Initialize()
{
	// If the object was created on a non-game thread, clear the async flag immediately, on both self, and the owned subobject, 
	// so that it can be garbage collected in the future. 
	(void)AtomicallyClearInternalFlags(EInternalObjectFlags::Async);
#if WITH_EDITOR	
	(void)PreallocatedMeshDescription->AtomicallyClearInternalFlags(EInternalObjectFlags::Async);
#endif
}


FSkeletalMeshSourceModel::FSkeletalMeshSourceModel() :
	Bounds(ForceInitToZero)
{
	
}


FSkeletalMeshSourceModel::FSkeletalMeshSourceModel(FSkeletalMeshSourceModel&& InSource)
{
	*this = MoveTemp(InSource);
}


FSkeletalMeshSourceModel& FSkeletalMeshSourceModel::operator=(FSkeletalMeshSourceModel&& InSource)
{
#if WITH_EDITOR
	RawMeshBulkData = MoveTemp(InSource.RawMeshBulkData);
	{
		// Ensure we lock both source data objects in the same order, to ensure we don't end up with a deadlock. 
		FScopeLock LockA(FMath::Max(&MeshDescriptionBulkDataMutex, &InSource.MeshDescriptionBulkDataMutex));
		FScopeLock LockB(FMath::Min(&MeshDescriptionBulkDataMutex, &InSource.MeshDescriptionBulkDataMutex));

		MeshDescriptionBulkData = InSource.MeshDescriptionBulkData;
		InSource.MeshDescriptionBulkData = nullptr;
	}
#endif
	
	return *this;
}

void FSkeletalMeshSourceModel::Initialize(USkeletalMesh* InOwner)
{
#if WITH_EDITOR
	FScopeLock Lock(&MeshDescriptionBulkDataMutex);

	if (!MeshDescriptionBulkData)
	{
		// Currently there is a restriction on creating UObjects on a thread while garbage collection is taking place,
		// so use a GCScopeGuard to prevent it happening at the same time.
		FGCScopeGuard Guard;
		MeshDescriptionBulkData = NewObject<USkeletalMeshDescriptionBulkData>(InOwner, NAME_None, RF_Transactional);
		MeshDescriptionBulkData->Initialize();
	}

	// We should not have a mesh description at this point.
	check(MeshDescriptionBulkData->GetMeshDescription() == nullptr);
#endif
}


TConstArrayView<FName> FSkeletalMeshSourceModel::GetSkinWeightProfileNames() const
{
	if (CachedSkinWeightProfileNames.Num() == 1 && CachedSkinWeightProfileNames[0] == NAME_None)
	{
#if WITH_EDITOR
		const_cast<FSkeletalMeshSourceModel*>(this)->UpdateCachedMeshStatisticsFromBulkIfNeeded();
#else
		static TArray<FName> EmptyArray;
		return EmptyArray; 
#endif
	}
	return CachedSkinWeightProfileNames;
}

TConstArrayView<FName> FSkeletalMeshSourceModel::GetMorphTargetNames() const
{
	if (CachedMorphTargetNames.Num() == 1 && CachedMorphTargetNames[0] == NAME_None)
	{
#if WITH_EDITOR
		const_cast<FSkeletalMeshSourceModel*>(this)->UpdateCachedMeshStatisticsFromBulkIfNeeded();
#else
		static TArray<FName> EmptyArray;
		return EmptyArray; 
#endif
	}
	return CachedMorphTargetNames;
}

TOptional<FGuid> FSkeletalMeshSourceModel::GetVertexColorGuid() const
{
#if WITH_EDITOR
	if (!CachedHasVertexColors.IsSet())
	{
		const_cast<FSkeletalMeshSourceModel*>(this)->UpdateCachedMeshStatisticsFromBulkIfNeeded();
	}
#endif

	if (CachedHasVertexColors.Get(false))
	{
		return CachedVertexColorGuid;
	}
	
	return {};
}


////////////////////////////////////////////////////////////////////////////////
///// EDITOR ONLY FUNCTIONS

#if WITH_EDITOR
bool FSkeletalMeshSourceModel::HasMeshDescription() const
{
	FScopeLock Lock(&MeshDescriptionBulkDataMutex);
	
	if (!ensure(MeshDescriptionBulkData != nullptr))
	{
		return false;
	}

	if (MeshDescriptionBulkData->IsBulkDataValid() || MeshDescriptionBulkData->HasCachedMeshDescription())
	{
		return true;
	}

	// If we have old raw data, then we _technically_ have a mesh description, since if LoadMeshDescription
	// gets called, this data will be converted to a mesh description.
	return RawMeshBulkData.IsValid();
}

FMeshDescription* FSkeletalMeshSourceModel::CreateMeshDescription()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FSkeletalMeshSourceData::CreateMeshDescription);
	FScopeLock Lock(&MeshDescriptionBulkDataMutex);
	
	if (!ensure(MeshDescriptionBulkData != nullptr))
	{
		return nullptr;
	}

	// Reset the old bulk data, since we won't be performing conversion on it.
	RawMeshBulkData = nullptr;

	return &MeshDescriptionBulkData->CreateMeshDescription()->GetMeshDescription();
}


FMeshDescription* FSkeletalMeshSourceModel::GetMeshDescription() const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FSkeletalMeshSourceData::GetMeshDescription);
	FScopeLock Lock(&MeshDescriptionBulkDataMutex);

	if (!ensure(MeshDescriptionBulkData))
	{
		return nullptr;
	}

	if (!MeshDescriptionBulkData->HasCachedMeshDescription())
	{
		FMeshDescription MeshDescription;
		if (LoadMeshDescriptionFromBulkData(MeshDescription))
		{
			UMeshDescriptionBase* MeshDescriptionBase = MeshDescriptionBulkData->CreateMeshDescription();
			MeshDescriptionBase->SetMeshDescription(MoveTemp(MeshDescription));
		}
	}
	
	if (MeshDescriptionBulkData->HasCachedMeshDescription())
	{
		FMeshDescription& MeshDescription = MeshDescriptionBulkData->GetMeshDescription()->GetMeshDescription();
		UpdateBonesDataIfNeeded(MeshDescription);
		return &MeshDescription;
	}

	return nullptr;
}


void FSkeletalMeshSourceModel::CommitMeshDescription(bool bInUseHashAsGuid)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FSkeletalMeshSourceData::CommitMeshDescription);
	FScopeLock Lock(&MeshDescriptionBulkDataMutex);

	if (!ensure(MeshDescriptionBulkData))
	{
		return;
	}
	
	if (MeshDescriptionBulkData->HasCachedMeshDescription())
	{
		MeshDescriptionBulkData->CommitMeshDescription(bInUseHashAsGuid);
		UpdateCachedMeshStatistics(&MeshDescriptionBulkData->GetMeshDescription()->GetMeshDescription());
	}
	else
	{
		MeshDescriptionBulkData->Empty();
		UpdateCachedMeshStatistics(nullptr);
	}
}


bool FSkeletalMeshSourceModel::CloneMeshDescription(FMeshDescription& OutMeshDescription) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FSkeletalMeshSourceData::CloneMeshDescription);
	FScopeLock Lock(&MeshDescriptionBulkDataMutex);

	if (!ensure(MeshDescriptionBulkData))
	{
		return false;
	}
	
	if (MeshDescriptionBulkData->HasCachedMeshDescription())
	{
		OutMeshDescription = MeshDescriptionBulkData->GetMeshDescription()->GetMeshDescription();
		return true;
	}

	return LoadMeshDescriptionFromBulkData(OutMeshDescription);
}


void FSkeletalMeshSourceModel::ClearMeshDescription()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FSkeletalMeshSourceData::ClearMeshDescription);
	FScopeLock Lock(&MeshDescriptionBulkDataMutex);

	if (ensure(MeshDescriptionBulkData))
	{
		MeshDescriptionBulkData->RemoveMeshDescription();
	}
}


void FSkeletalMeshSourceModel::ClearAllMeshData()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FSkeletalMeshSourceData::ClearAllMeshData);
	FScopeLock Lock(&MeshDescriptionBulkDataMutex);

	if (ensure(MeshDescriptionBulkData))
	{
		MeshDescriptionBulkData->RemoveMeshDescription();
		MeshDescriptionBulkData->Empty();
	}
}

const FMeshDescriptionBulkData* FSkeletalMeshSourceModel::GetMeshDescriptionBulkData() const
{
	FScopeLock Lock(&MeshDescriptionBulkDataMutex);
	if (!ensure(MeshDescriptionBulkData))
	{
		return nullptr;
	}

	return &MeshDescriptionBulkData->GetBulkData();
}


USkeletalMesh* FSkeletalMeshSourceModel::GetOwner() const
{
	if (!ensure(MeshDescriptionBulkData))
	{
		return nullptr;
	}

	return Cast<USkeletalMesh>(MeshDescriptionBulkData->GetOuter());
}

void FSkeletalMeshSourceModel::EnsureRawMeshBulkDataIsConvertedToNew()
{
	if (!ensure(MeshDescriptionBulkData))
	{
		return;
	}

	if (!MeshDescriptionBulkData->IsBulkDataValid() && RawMeshBulkData.IsValid())
	{
		ConvertRawMeshToMeshDescriptionBulkData();
	}
}


bool FSkeletalMeshSourceModel::LoadMeshDescriptionFromBulkData(
	FMeshDescription& OutMeshDescription
	) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FSkeletalMeshSourceData::LoadMeshDescriptionFromBulkData);
	
	// Internal function, we assume MeshDescriptionBulkDataMutex is held.
	if (!ensure(MeshDescriptionBulkData != nullptr))
	{
		return false;
	}

	if (!MeshDescriptionBulkData->IsBulkDataValid() && RawMeshBulkData.IsValid())
	{
		const_cast<FSkeletalMeshSourceModel*>(this)->ConvertRawMeshToMeshDescriptionBulkData();
	}

	if (!MeshDescriptionBulkData->IsBulkDataValid())
	{
		return false;
	}
	
	MeshDescriptionBulkData->GetBulkData().LoadMeshDescription(OutMeshDescription);
	
	// If this mesh is stored with the older representation, update it now.
	UpgradeMorphTargets(OutMeshDescription);
	
	return true;
}

void FSkeletalMeshSourceModel::UpgradeMorphTargets(
	FMeshDescription& InOutMeshDescription
	)	
{
	TArray<FName> AttributeNames;
	InOutMeshDescription.VertexAttributes().GetAttributeNames(AttributeNames);

	TArray<FVector3f> SavedPointDelta;
	for (const FName AttributeName: AttributeNames)
	{
		if (!FSkeletalMeshAttributes::IsMorphTargetAttribute(AttributeName))
		{
			continue;
		}
		
		TVertexAttributesConstRef<TArrayView<FVector3f>> OldMorphTarget = InOutMeshDescription.VertexAttributes().GetAttributesRef<TArrayView<FVector3f>>(AttributeName);

		if (!OldMorphTarget.IsValid())
		{
			continue;
		}
		// Grab all the points and store them away. We then re-register the attribute(s) with the correct details. We only
		// need the point deltas, since that's all that was stored before the storage change.
		SavedPointDelta.SetNumUninitialized(InOutMeshDescription.Vertices().GetArraySize());

		for (FVertexID VertexID: InOutMeshDescription.Vertices().GetElementIDs())
		{
			SavedPointDelta[VertexID.GetValue()] = OldMorphTarget.Get(VertexID)[0];
		}

		TVertexAttributesRef<FVector3f> PointDelta = InOutMeshDescription.VertexAttributes().RegisterAttribute<FVector3f>(AttributeName, 1, FVector3f::ZeroVector, EMeshAttributeFlags::None);
		for (FVertexID VertexID: InOutMeshDescription.Vertices().GetElementIDs())
		{
			PointDelta.Set(VertexID, SavedPointDelta[VertexID.GetValue()]);
		}
	}
}


void FSkeletalMeshSourceModel::ConvertRawMeshToMeshDescriptionBulkData()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FSkeletalMeshSourceData::ConvertRawMeshToMeshDescriptionBulkData);
	
	FSkeletalMeshImportData ImportData;
	RawMeshBulkData->LoadRawMesh(ImportData);

	// In some cases, FSkeletalMeshImportData::MeshInfos was not baked in during import, but only in FSkeletalMeshLODModel::ImportedMeshInfos.
	// In that case we use the FSkeletalMeshLODModel::ImportedMeshInfos as a source of ground truth and bake it into the mesh description.
	const USkeletalMesh* SkeletalMesh = GetOwner();
	if (SkeletalMesh && SkeletalMesh->GetImportedModel() && SkeletalMesh->GetImportedModel()->LODModels.IsValidIndex(RawMeshBulkDataLODIndex))
	{
		const FSkeletalMeshLODModel& Model = SkeletalMesh->GetImportedModel()->LODModels[RawMeshBulkDataLODIndex];
		if (!Model.ImportedMeshInfos.IsEmpty() && ImportData.MeshInfos.IsEmpty())
		{
			for (const FSkeletalMeshLODModel::FSkelMeshImportedMeshInfo& SourceInfo: Model.ImportedMeshInfos)
			{
				SkeletalMeshImportData::FMeshInfo& TargetInfo = ImportData.MeshInfos.AddDefaulted_GetRef();
				TargetInfo.Name = SourceInfo.Name;
				TargetInfo.StartImportedVertex = SourceInfo.StartImportedVertex;
				TargetInfo.NumVertices = SourceInfo.NumVertices;
			}
		}
	}
	
	FMeshDescription& MeshDescription = MeshDescriptionBulkData->CreateMeshDescription()->GetMeshDescription(); 
	ImportData.GetMeshDescription(SkeletalMesh, &SkeletalMesh->GetLODInfo(RawMeshBulkDataLODIndex)->BuildSettings, MeshDescription);

	UpdateCachedMeshStatistics(&MeshDescription);

	constexpr bool bUseHashAsGuid = true;
	MeshDescriptionBulkData->CommitMeshDescription(bUseHashAsGuid);

	// Ensure we don't re-check for raw mesh conversion.
	RawMeshBulkData = nullptr;
}

void FSkeletalMeshSourceModel::UpdateCachedMeshStatistics(const FMeshDescription* InMeshDescription)
{
	if (InMeshDescription)
	{
		// Save out the triangle/vertex counts and bounds for fast lookup. 
		TriangleCount = InMeshDescription->Triangles().Num();
		VertexCount = InMeshDescription->Vertices().Num();
		Bounds = InMeshDescription->GetBounds();
		
		FSkeletalMeshConstAttributes Attributes(*InMeshDescription);
		constexpr bool bInUserDefinedOnly = true;
		CachedSkinWeightProfileNames = Attributes.GetSkinWeightProfileNames(bInUserDefinedOnly);
		CachedMorphTargetNames = Attributes.GetMorphTargetNames();

		// Test whether there are non-zero vertex colors, and compute a pseudo-hash for them to ensure that
		// each time the vertex colors change, we have a new GUID to ensure the vertex data gets updated. 
		struct FContext
		{
			uint32 Parts[4] = {0};
			bool bHasVertexColors = false;
		};
		TArray<FContext> VertexColorContexts;
		
		TVertexInstanceAttributesConstRef<FVector4f> ColorAttrib = Attributes.GetVertexInstanceColors();
		const FVertexInstanceArray& VertexInstances = InMeshDescription->VertexInstances();
		ParallelForWithTaskContext(VertexColorContexts, VertexInstances.GetArraySize(), [&ColorAttrib, VertexInstances](FContext &InOutContext, int32 InIndex)
		{
			const FVertexInstanceID VertexInstanceID(InIndex);
			if (VertexInstances.IsValid(VertexInstanceID))
			{
				const FVector4f VertexColor = ColorAttrib.Get(VertexInstanceID);

				if (!VertexColor.IsNearlyZero())
				{
					InOutContext.bHasVertexColors = true;
				}

				// Build a pseudo-hash for the vertex colors, in a thread-safe manner by just adding up the float bits. This will be consistent
				// no matter how the vertex instances are partitioned across threads. 
				InOutContext.Parts[0] += *reinterpret_cast<const uint32*>(&VertexColor.X) + static_cast<uint32>(InIndex);
				InOutContext.Parts[1] += *reinterpret_cast<const uint32*>(&VertexColor.Y) + static_cast<uint32>(InIndex);
				InOutContext.Parts[2] += *reinterpret_cast<const uint32*>(&VertexColor.Z) + static_cast<uint32>(InIndex);
				InOutContext.Parts[3] += *reinterpret_cast<const uint32*>(&VertexColor.W) + static_cast<uint32>(InIndex);
			}
		});

		uint32 Parts[4] = {0};
		bool bHasVertexColors = false;
		for (const FContext& Context: VertexColorContexts)
		{
			if (Context.bHasVertexColors)
			{
				bHasVertexColors = true;
			}
			Parts[0] += Context.Parts[0];
			Parts[1] += Context.Parts[1];
			Parts[2] += Context.Parts[2];
			Parts[3] += Context.Parts[3];
		}
		
		CachedHasVertexColors = bHasVertexColors;
		CachedVertexColorGuid = FGuid(Parts[0], Parts[1], Parts[2], Parts[3]);
	}
	else
	{
		TriangleCount = VertexCount = 0;
		Bounds = FBoxSphereBounds{ForceInitToZero};
		CachedSkinWeightProfileNames.Reset();
		CachedMorphTargetNames.Reset();
		CachedHasVertexColors.Reset();
		CachedVertexColorGuid = {};
	}
}

void FSkeletalMeshSourceModel::UpdateCachedMeshStatisticsFromBulkIfNeeded()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FSkeletalMeshSourceData::UpdateCachedMeshStatisticsFromBulkIfNeeded);

	FScopeLock Lock(&MeshDescriptionBulkDataMutex);

	if (!ensure(MeshDescriptionBulkData))
	{
		return;
	}
	
	if (MeshDescriptionBulkData->HasCachedMeshDescription())
	{
		UpdateCachedMeshStatistics(&MeshDescriptionBulkData->GetMeshDescription()->GetMeshDescription());
	}

	if (!MeshDescriptionBulkData->IsBulkDataValid())
	{
		if (RawMeshBulkData.IsValid())
		{
			// This call will implicitly update the mesh statistics.
			ConvertRawMeshToMeshDescriptionBulkData();
		}
		else
		{
			UpdateCachedMeshStatistics(nullptr);
		}
	}
	else
	{
		// Temporarily load the mesh description from the bulk data, we don't care about fixing up the morph targets
		// in this instance (as would be the case with LoadMeshDescriptionFromBulkData).
		FMeshDescription MeshDescription;
		MeshDescriptionBulkData->GetBulkData().LoadMeshDescription(MeshDescription);

		UpdateCachedMeshStatistics(&MeshDescription);
	}
}

void FSkeletalMeshSourceModel::UpdateBonesDataIfNeeded(FMeshDescription& InOutMeshDescription) const
{
	FSkeletalMeshAttributes MeshAttributes(InOutMeshDescription);
	if (!MeshAttributes.HasBones())
	{
		MeshAttributes.Register(true);
	}

	const USkeletalMesh* SkeletalMesh = GetOwner();
	if (!SkeletalMesh)
	{
		return;
	}
	
	const int32 NumMeshDescBones = MeshAttributes.GetNumBones();
	const FReferenceSkeleton RefSkeleton = SkeletalMesh->GetRefSkeleton();
	const int32 NumRefBones = SkeletalMesh->GetRefSkeleton().GetRawBoneNum();
	if (NumMeshDescBones != NumRefBones && NumRefBones > 0)
	{
		MeshAttributes.Bones().Reset(NumRefBones);
	
		FSkeletalMeshAttributes::FBoneNameAttributesRef BoneNames = MeshAttributes.GetBoneNames();
		FSkeletalMeshAttributes::FBoneParentIndexAttributesRef BoneParentIndices = MeshAttributes.GetBoneParentIndices();
		FSkeletalMeshAttributes::FBonePoseAttributesRef BonePoses = MeshAttributes.GetBonePoses();


		for (int Index = 0; Index < NumRefBones; ++Index)
		{
			const FMeshBoneInfo& BoneInfo = RefSkeleton.GetRawRefBoneInfo()[Index];
			const FTransform& BoneTransform = RefSkeleton.GetRawRefBonePose()[Index];

			const FBoneID BoneID = MeshAttributes.CreateBone();

			BoneNames.Set(BoneID, BoneInfo.Name);
			BoneParentIndices.Set(BoneID, BoneInfo.ParentIndex);
			BonePoses.Set(BoneID, BoneTransform);
		}
	}
}

#endif
