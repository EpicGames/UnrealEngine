// Copyright Epic Games, Inc. All Rights Reserved.

#include "SkeletalMeshUpdater.h"
#include "GPUSkinCache.h"
#include "SceneInterface.h"
#include "Rendering/RenderCommandPipes.h"
#include "RenderGraphBuilder.h"

#if WITH_STATE_STREAM
#define SKELETAL_MESH_UPDATER_CHECK_THREAD_ACCESS check(IsInParallelRenderingThread());
#else
// Producers (Create/Update/Release) and PopFromQueue are synchronized via OpQueueLock (FRWLock).
// Producers take a shared lock; PopFromQueue takes an exclusive lock.
// No thread restriction needed - generic workers can safely produce ops.
#define SKELETAL_MESH_UPDATER_CHECK_THREAD_ACCESS
#endif


int32 GSkeletalMeshDynamicDataPoolBudget = 4 * 1024;
static FAutoConsoleVariableRef CVarSkeletalMeshDynamicDataPoolSize(
	TEXT("r.SkeletalMesh.DynamicDataPoolBudget"),
	GSkeletalMeshDynamicDataPoolBudget,
	TEXT("Controls the maximum size of the skeletal mesh dynamic data pools, in KB. A value <=0 disables the pool.\n"),
	ECVF_RenderThreadSafe);

const FSkeletalMeshUpdater::FSubmitData FSkeletalMeshUpdater::FSubmitData::EmptyData;

int64 FSkeletalMeshDynamicDataPool::GetPoolBudget()
{
	return FMath::Max(0, GSkeletalMeshDynamicDataPoolBudget * 1024);
}

///////////////////////////////////////////////////////////////////////////////

int32 FSkeletalMeshDynamicDataPool::GetBucketIndex(int32 NumTransforms) const
{
	static_assert(NumPoolBuckets == 5);
	if (NumTransforms <= 64)
	{
		return 4;
	}
	else if (NumTransforms <= 128)
	{
		return 3;
	}
	else if (NumTransforms <= 256)
	{
		return 2;
	}
	else if (NumTransforms <= 512)
	{
		return 1;
	}
	return 0;
}

#if COUNTERSTRACE_ENABLED

TRACE_DECLARE_ATOMIC_MEMORY_COUNTER(SkeletalMesh_DynamicDataPool_Size, TEXT("SkeletalMesh/DynamicDataPool/Size"));
TRACE_DECLARE_ATOMIC_MEMORY_COUNTER(SkeletalMesh_DynamicDataPool_Bucket0, TEXT("SkeletalMesh/DynamicDataPool/Bucket0"));
TRACE_DECLARE_ATOMIC_MEMORY_COUNTER(SkeletalMesh_DynamicDataPool_Bucket1, TEXT("SkeletalMesh/DynamicDataPool/Bucket1"));
TRACE_DECLARE_ATOMIC_MEMORY_COUNTER(SkeletalMesh_DynamicDataPool_Bucket2, TEXT("SkeletalMesh/DynamicDataPool/Bucket2"));
TRACE_DECLARE_ATOMIC_MEMORY_COUNTER(SkeletalMesh_DynamicDataPool_Bucket3, TEXT("SkeletalMesh/DynamicDataPool/Bucket3"));
TRACE_DECLARE_ATOMIC_MEMORY_COUNTER(SkeletalMesh_DynamicDataPool_Bucket4, TEXT("SkeletalMesh/DynamicDataPool/Bucket4"));

void FSkeletalMeshDynamicDataPool::AddStatsMemory(int32 BucketIndex, int32 Size)
{
	if (!UE_TRACE_CHANNELEXPR_IS_ENABLED(CountersChannel))
	{
		return;
	}

	TRACE_COUNTER_ADD(SkeletalMesh_DynamicDataPool_Size, Size);

	switch (BucketIndex)
	{
	case 0:
		TRACE_COUNTER_ADD(SkeletalMesh_DynamicDataPool_Bucket0, Size);
		break;
	case 1:
		TRACE_COUNTER_ADD(SkeletalMesh_DynamicDataPool_Bucket1, Size);
		break;
	case 2:
		TRACE_COUNTER_ADD(SkeletalMesh_DynamicDataPool_Bucket2, Size);
		break;
	case 3:
		TRACE_COUNTER_ADD(SkeletalMesh_DynamicDataPool_Bucket3, Size);
		break;
	case 4:
		TRACE_COUNTER_ADD(SkeletalMesh_DynamicDataPool_Bucket4, Size);
		break;
	}
}

#endif

///////////////////////////////////////////////////////////////////////////////

void FSkeletalMeshUpdatePacket::Init(FSceneInterface* InScene, FGPUSkinCache* InGPUSkinCache, ERHIPipeline InGPUSkinCachePipeline, const FInitializer& Initializer)
{
	GPUSkinCache = InGPUSkinCache;
	GPUSkinCachePipeline = InGPUSkinCachePipeline;
	Scene = InScene;

#if RHI_RAYTRACING
	bSkinCacheForRayTracingSupported = GPUSkinCache && FGPUSkinCache::IsGPUSkinCacheRayTracingSupported(GPUSkinCache->GetFeatureLevel());
#endif

	Init(Initializer);
}

void FSkeletalMeshUpdatePacket::Finalize()
{
#if RHI_RAYTRACING
	if (bInvalidatePathTracedOutput)
	{
		Scene->InvalidatePathTracedOutput();
	}
#endif
}

///////////////////////////////////////////////////////////////////////////////

FSkeletalMeshUpdateChannel::FBackend::FGlobalList& FSkeletalMeshUpdateChannel::FBackend::GetGlobalList()
{
	static FGlobalList GlobalList;
	return GlobalList;
}

FSkeletalMeshUpdateChannel::FBackend::FBackend()
{
	FGlobalList& GlobalList = GetGlobalList();
	GlobalList.List.AddTail(this);
	GlobalListIndex = GlobalList.Num++;
}

FSkeletalMeshUpdateChannel::FBackend::~FBackend()
{
	for (FSkeletalMeshUpdateChannel* Channel : Channels)
	{
		Channel->Backend = nullptr;
	}
	Channels.Empty();
	Reset();
}

///////////////////////////////////////////////////////////////////////////////

int32 FSkeletalMeshUpdateChannel::FIndexAllocator::Allocate()
{
	UE::TScopeLock Lock(Mutex);
	if (!FreeList.IsEmpty())
	{
		return FreeList.Pop(EAllowShrinking::No);
	}
	return Max++;
}

void FSkeletalMeshUpdateChannel::FIndexAllocator::Free(int32 Index)
{
	UE::TScopeLock Lock(Mutex);
	FreeList.Push(Index);
}

///////////////////////////////////////////////////////////////////////////////

TArray<FSkeletalMeshUpdateChannel> FSkeletalMeshUpdateChannel::GetChannels()
{
	FBackend::FGlobalList& GlobalList = FBackend::GetGlobalList();

	TArray<FSkeletalMeshUpdateChannel> Channels;
	Channels.Reserve(GlobalList.Num);

	int32 ChannelIndex = 0;
	for (FBackend& Backend : GlobalList.List)
	{
		// Hitting this means something went wrong with static initialization.
		check(ChannelIndex++ == Backend.GlobalListIndex);

		Channels.Emplace(&Backend);
	}

	return Channels;
}

FSkeletalMeshUpdateChannel::FSkeletalMeshUpdateChannel(FBackend* InBackend)
	: OpQueue(new FOpQueue)
	, Backend(InBackend)
{
	check(Backend);
	Backend->Channels.Emplace(this);
}

FSkeletalMeshUpdateChannel::~FSkeletalMeshUpdateChannel()
{
	// Backend can be nulled out first.
	if (Backend)
	{
		Backend->Channels.Remove(this);
	}

	check(!OpQueue);
	check(OpStream.Ops.IsEmpty());

	const int32 NumAllocatedHandles = IndexAllocator.NumAllocated();
	checkf(NumAllocatedHandles == 0, TEXT("FSkeletalMeshUpdateChannel is destructing but still has %d valid handles!"), NumAllocatedHandles);
}

FSkeletalMeshUpdateHandle FSkeletalMeshUpdateChannel::Create(FSkeletalMeshObject* MeshObject)
{
	SKELETAL_MESH_UPDATER_CHECK_THREAD_ACCESS

	FSkeletalMeshUpdateHandle Handle;
	Handle.Index = IndexAllocator.Allocate();
	Handle.Channel = this;

	FOp Op;
	Op.HandleIndex = Handle.Index;
	Op.Type = FOp::EType::Add;
	Op.Data_Add.MeshObject = MeshObject;

	// Shared lock: prevents PopFromQueue from swapping the OpQueue pointer while we enqueue.
	// Multiple producers can enqueue concurrently (MPSC queue + atomics are thread-safe).
	FReadScopeLock ReadLock(OpQueueLock);
	check(OpQueue);
	OpQueue->Queue.Enqueue(Op);
	OpQueue->NumAdds.fetch_add(1, std::memory_order_relaxed);
	OpQueue->Num.fetch_add(1, std::memory_order_relaxed);

	return Handle;
}

void FSkeletalMeshUpdateChannel::Release(FSkeletalMeshUpdateHandle&& Handle)
{
	SKELETAL_MESH_UPDATER_CHECK_THREAD_ACCESS
	check(Handle.Channel == this);

	FOp Op;
	Op.HandleIndex = Handle.Index;
	Op.Type = FOp::EType::Remove;

	{
		// Shared lock: prevents PopFromQueue from swapping the OpQueue pointer while we enqueue.
		FReadScopeLock ReadLock(OpQueueLock);
		check(OpQueue);
		OpQueue->Queue.Enqueue(Op);
		OpQueue->NumRemoves.fetch_add(1, std::memory_order_relaxed);
		OpQueue->Num.fetch_add(1, std::memory_order_relaxed);
	}

	IndexAllocator.Free(Handle.Index);

	// Clear the channel so that the handle can destruct cleanly.
	Handle.Channel = nullptr;
}

void FSkeletalMeshUpdateChannel::Update(const FSkeletalMeshUpdateHandle& Handle, FSkeletalMeshDynamicData* MeshDynamicData)
{
	SKELETAL_MESH_UPDATER_CHECK_THREAD_ACCESS
	check(MeshDynamicData);
	check(Handle.Channel == this);

	FOp Op;
	Op.HandleIndex = Handle.Index;
	Op.Type = FOp::EType::Update;
	Op.Data_Update.MeshDynamicData = MeshDynamicData;

	// Shared lock: prevents PopFromQueue from swapping the OpQueue pointer while we enqueue.
	FReadScopeLock ReadLock(OpQueueLock);
	check(OpQueue);
	OpQueue->Queue.Enqueue(Op);
	OpQueue->NumUpdates.fetch_add(1, std::memory_order_relaxed);
	OpQueue->Num.fetch_add(1, std::memory_order_relaxed);
}

void FSkeletalMeshUpdateChannel::Shutdown()
{
	// Exclusive lock: waits for all producers to finish before nulling the queue.
	FWriteScopeLock WriteLock(OpQueueLock);
	OpQueue = nullptr;
}

TUniquePtr<FSkeletalMeshUpdateChannel::FOpQueue> FSkeletalMeshUpdateChannel::PopFromQueue()
{
	SKELETAL_MESH_UPDATER_CHECK_THREAD_ACCESS

	// Exclusive lock: waits for all producers to finish enqueuing before swapping the queue.
	// Without this, a producer could be mid-Enqueue on the old queue when it's passed to
	// the render thread for Close(), losing the op.
	FWriteScopeLock WriteLock(OpQueueLock);
	if (!OpQueue || OpQueue->Num.load(std::memory_order_relaxed) == 0)
	{
		return {};
	}

	TUniquePtr<FOpQueue> Data(MoveTemp(OpQueue));
	OpQueue = MakeUnique<FOpQueue>();
	return Data;
}

void FSkeletalMeshUpdateChannel::PushToStream(TUniquePtr<FSkeletalMeshUpdateChannel::FOpQueue>&& InOps)
{
	check(IsInRenderingThread());

	OpStream.NumAdds    += InOps->NumAdds.load(std::memory_order_relaxed);
	OpStream.NumRemoves += InOps->NumRemoves.load(std::memory_order_relaxed);
	OpStream.NumUpdates += InOps->NumUpdates.load(std::memory_order_relaxed);
	OpStream.Num        += InOps->Num.load(std::memory_order_relaxed);
	OpStream.Ops.Reserve(OpStream.Num);

	InOps->Queue.Close([&] (const FOp& Op)
	{
		OpStream.Ops.Emplace(Op);
	});
}

///////////////////////////////////////////////////////////////////////////////

FSkeletalMeshUpdater::FSkeletalMeshUpdater(FSceneInterface* InScene, FGPUSkinCache* InGPUSkinCache)
	: Scene(InScene)
	, GPUSkinCache(InGPUSkinCache)
	, Channels(FSkeletalMeshUpdateChannel::GetChannels())
{
#if !WITH_STATE_STREAM
	DelegateHandle = UE::RenderCommandPipe::GetStopRecordingDelegate().AddLambda([this] (const FRenderCommandPipeBitArray&)
	{
		if (bInAsyncPushCommandsRegion)
		{
			return;
		}

		TArray<TPair<FSkeletalMeshUpdateChannel*, TUniquePtr<FSkeletalMeshUpdateChannel::FOpQueue>>, FConcurrentLinearArrayAllocator> ChannelsToPush;
		ChannelsToPush.Reserve(Channels.Num());

		for (FSkeletalMeshUpdateChannel& Channel : Channels)
		{
			if (TUniquePtr<FSkeletalMeshUpdateChannel::FOpQueue> Ops{ Channel.PopFromQueue() }; Ops.IsValid())
			{
				ChannelsToPush.Emplace(&Channel, MoveTemp(Ops));
			}
		}

		if (!Channels.IsEmpty())
		{
			ENQUEUE_RENDER_COMMAND(FSkeletalMeshUpdater_PopFromQueues)([this, ChannelsToPush = MoveTemp(ChannelsToPush)](FRHICommandList& RHICmdList) mutable
			{
				for (auto& KeyValue : ChannelsToPush)
				{
					KeyValue.Key->PushToStream(MoveTemp(KeyValue.Value));
				}
			});
		}
	});
#endif
}

void FSkeletalMeshUpdater::Shutdown()
{
	UE::RenderCommandPipe::GetStopRecordingDelegate().Remove(DelegateHandle);

	for (FSkeletalMeshUpdateChannel& Channel : Channels)
	{
		Channel.Shutdown();
	}
}

void FSkeletalMeshUpdater::PushChangesToStream()
{
	TArray<TPair<FSkeletalMeshUpdateChannel*, TUniquePtr<FSkeletalMeshUpdateChannel::FOpQueue>>, FConcurrentLinearArrayAllocator> ChannelsToPush;
	ChannelsToPush.Reserve(Channels.Num());

	for (FSkeletalMeshUpdateChannel& Channel : Channels)
	{
		if (TUniquePtr<FSkeletalMeshUpdateChannel::FOpQueue> Ops{ Channel.PopFromQueue() }; Ops.IsValid())
		{
			ChannelsToPush.Emplace(&Channel, MoveTemp(Ops));
		}
	}

	for (auto& KeyValue : ChannelsToPush)
	{
		KeyValue.Key->PushToStream(MoveTemp(KeyValue.Value));
	}
}

UE::Tasks::FTask FSkeletalMeshUpdater::AddPushCommandsTask(const UE::Tasks::FTask& PrerequisiteTask)
{
	check(bInAsyncPushCommandsRegion);

	using FChannelsToPush = TArray<TPair<FSkeletalMeshUpdateChannel*, TUniquePtr<FSkeletalMeshUpdateChannel::FOpQueue>>, FConcurrentLinearArrayAllocator>;

	FChannelsToPush* ChannelsToPush = new FChannelsToPush();

	PushCommandsTask = UE::Tasks::Launch(UE_SOURCE_LOCATION, [this, ChannelsToPush]
	{
		FTaskTagScope TagScope(ETaskTag::EParallelGameThread);
		ChannelsToPush->Reserve(Channels.Num());

		for (FSkeletalMeshUpdateChannel& Channel : Channels)
		{
			if (TUniquePtr<FSkeletalMeshUpdateChannel::FOpQueue> Ops{ Channel.PopFromQueue() }; Ops.IsValid())
			{
				ChannelsToPush->Emplace(&Channel, MoveTemp(Ops));
			}
		}

	}, MakeArrayView({ PushCommandsTask, PrerequisiteTask }), UE::Tasks::ETaskPriority::Default, UE::Tasks::EExtendedTaskPriority::Inline);

	ENQUEUE_RENDER_COMMAND(FSkeletalMeshUpdater_PopFromQueues)([this, ChannelsToPush, ChannelsToPushTask = PushCommandsTask](FRHICommandList& RHICmdList) mutable
	{
		ChannelsToPushTask.Wait();
		for (auto& KeyValue : *ChannelsToPush)
		{
			KeyValue.Key->PushToStream(MoveTemp(KeyValue.Value));
		}
		delete ChannelsToPush;
	});

	return PushCommandsTask;
}

RDG_REGISTER_BLACKBOARD_STRUCT(FSkeletalMeshUpdater::FTaskData);

struct FSkeletalMeshUpdater::FTaskData : public FSkeletalMeshUpdater::FSubmitData
{
	FTaskData(FRDGBuilder& GraphBuilder, ERHIPipeline InGPUSkinCachePipeline)
		: GPUSkinCachePipeline(InGPUSkinCachePipeline)
		, bAsyncCommandList(GraphBuilder.IsParallelSetupEnabled())
	{
		Tasks.Filter       = TaskEvents.Filter;
		Tasks.Inline       = TaskEvents.Inline;
		Tasks.MeshDeformer = TaskEvents.MeshDeformer;
		Tasks.SkinCache    = TaskEvents.SkinCache;

		if (bAsyncCommandList)
		{
			RHICmdList = new FRHICommandList;

			FRHICommandListScopedPipeline ScopedPipeline(GraphBuilder.RHICmdList, GPUSkinCachePipeline);
			GraphBuilder.RHICmdList.QueueAsyncCommandListSubmit(RHICmdList);
		}
		else
		{
			RHICmdList = &GraphBuilder.RHICmdList;
		}
	}

	void Begin(int32 NumChannels)
	{
		Packets.Reserve(NumChannels);

		if (bAsyncCommandList)
		{
			RHICmdList->SwitchPipeline(ERHIPipeline::Graphics);
		}

		RHICmdListScopedFence.Emplace(*RHICmdList);
	}

	void End()
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(Cleanup);
		RHICmdListScopedFence.Reset();

		if (bAsyncCommandList)
		{
			RHICmdList->FinishRecording();
		}

		for (auto& UpdatePacket : Packets)
		{
			UpdatePacket->TrimPool();
		}
		Packets.Empty();
	}

	void ProcessForeground()
	{
		TaskEvents.Filter.Trigger();

		{
			TRACE_CPUPROFILER_EVENT_SCOPE(Inline);
			for (auto& UpdatePacket : Packets)
			{
				UpdatePacket->ProcessStage_Inline(*RHICmdList, TaskEvents.Inline);
			}
			TaskEvents.Inline.Trigger();
		}

		{
			TRACE_CPUPROFILER_EVENT_SCOPE(MeshDeformer);
			for (auto& UpdatePacket : Packets)
			{
				UpdatePacket->ProcessStage_MeshDeformer(*RHICmdList, TaskEvents.MeshDeformer);
			}
			TaskEvents.MeshDeformer.Trigger();
		}
	
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(GPUSkinCache);
			for (auto& UpdatePacket : Packets)
			{
				UpdatePacket->ProcessStage_SkinCache(*RHICmdList, TaskEvents.SkinCache);
			}
			TaskEvents.SkinCache.Trigger();
		}
	}

	void ProcessUpload()
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FSkeletalMeshUpdater::ProcessUpload);

		for (auto& UpdatePacket : Packets)
		{
			UpdatePacket->ProcessStage_Upload(*RHICmdList);
		}
	}

	TArray<TUniquePtr<FSkeletalMeshUpdatePacket>, FConcurrentLinearArrayAllocator> Packets;
	TOptional<FRHICommandListScopedFence> RHICmdListScopedFence;
	FRHICommandList* RHICmdList;
	const ERHIPipeline GPUSkinCachePipeline;
	const bool bAsyncCommandList;

	struct
	{
		UE::Tasks::FTaskEvent Filter{ UE_SOURCE_LOCATION };
		UE::Tasks::FTaskEvent Inline{ UE_SOURCE_LOCATION };
		UE::Tasks::FTaskEvent MeshDeformer{ UE_SOURCE_LOCATION };
		UE::Tasks::FTaskEvent SkinCache{ UE_SOURCE_LOCATION };

	} TaskEvents;
};

const FSkeletalMeshUpdater::FSubmitData& FSkeletalMeshUpdater::Submit(FRDGBuilder& GraphBuilder, ERHIPipeline GPUSkinCachePipeline)
{
	bool bAllOpsEmpty = true;

	for (FSkeletalMeshUpdateChannel& Channel : Channels)
	{
		if (Channel.OpStream.Num > 0)
		{
			bAllOpsEmpty = false;
			break;
		}
	}

	if (bAllOpsEmpty)
	{
		return FSubmitData::EmptyData;
	}

	checkf(!bSubmitting, TEXT("Submit was called twice on the same RDG builder. This is not allowed."));
	bSubmitting = true;

	FTaskData& TaskData = GraphBuilder.Blackboard.Create<FTaskData>(GraphBuilder, GPUSkinCachePipeline);

	UE::Tasks::FTask ForegroundTask = GraphBuilder.AddSetupTask([this, &TaskData]
	{
		TaskData.Begin(Channels.Num());

		for (FSkeletalMeshUpdateChannel& Channel : Channels)
		{
			TUniquePtr<FSkeletalMeshUpdatePacket> Packet = Channel.CreatePacket();

			Packet->Init(Scene, GPUSkinCache, TaskData.GPUSkinCachePipeline, Channel.GetPacketInitializer());
			Channel.Replay(*TaskData.RHICmdList, *Packet, TaskData.UpdateEvents);
			Packet->Finalize();

			TaskData.Packets.Emplace(MoveTemp(Packet));
		}

		TaskData.ProcessForeground();

	}, UE::Tasks::ETaskPriority::High);

	GraphBuilder.AddSetupTask([this, &TaskData]
	{
		TaskData.ProcessUpload();
		TaskData.End();

		bSubmitting = false;

	}, ForegroundTask, UE::Tasks::ETaskPriority::BackgroundHigh);

	return TaskData;
}

void FSkeletalMeshUpdater::WaitForStage(FRDGBuilder& GraphBuilder, ESkeletalMeshUpdateStage Stage)
{
	if (FTaskData* TaskData = GraphBuilder.Blackboard.GetMutable<FTaskData>())
	{
		switch (Stage)
		{
		default: checkNoEntry();
		case ESkeletalMeshUpdateStage::Filter:
			TaskData->Tasks.Filter.Wait();
			TaskData->Tasks.Filter = {};
			break;
		case ESkeletalMeshUpdateStage::Inline:
			TaskData->Tasks.Inline.Wait();
			TaskData->Tasks.Inline = {};
			break;
		case ESkeletalMeshUpdateStage::MeshDeformer:
			TaskData->Tasks.MeshDeformer.Wait();
			TaskData->Tasks.MeshDeformer = {};
			break;
		case ESkeletalMeshUpdateStage::SkinCache:
			TaskData->Tasks.SkinCache.Wait();
			TaskData->Tasks.SkinCache = {};
			break;
		}
	}
}