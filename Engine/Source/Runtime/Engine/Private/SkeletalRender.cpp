// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	SkeletalRender.cpp: Skeletal mesh skinning/rendering code.
=============================================================================*/

#include "SkeletalRender.h"
#include "Engine/SkinnedAsset.h"
#include "Engine/SkinnedAssetCommon.h"
#include "SkeletalRenderPublic.h"
#include "PrimitiveDrawingUtils.h"
#include "SceneView.h"
#include "GPUSkinCache.h"
#include "Rendering/SkeletalMeshRenderData.h"
#include "Engine/RendererSettings.h"
#include "SkinnedMeshSceneProxyDesc.h"
#include "SkinningSceneExtensionProxy.h"

/*-----------------------------------------------------------------------------
Globals
-----------------------------------------------------------------------------*/

// smallest blend weight for vertex anims
const float MinMorphTargetBlendWeight = UE_SMALL_NUMBER;
// largest blend weight for vertex anims
const float MaxMorphTargetBlendWeight = 5.0f;

static float GMorphTargetMaxBlendWeight = 5.f;
static FAutoConsoleVariableRef CVarMorphTargetMinBlendWeight(
	TEXT("r.MorphTarget.MaxBlendWeight"),
	GMorphTargetMaxBlendWeight,
	TEXT("Maximum value accepted as a morph target blend weight..\n")
	TEXT("Blend target weights will be checked against this value for validation.Values smaller than this number will be clamped.\n"),
	ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<bool> CVarSkinnedMeshSkinWeightProfilesOnRenderThread(
	TEXT("r.SkinnedMesh.SkinWeightProfilesOnRenderThread"),
	false,
	TEXT("Do all skin weight profile management on the render thread (experimental / disabled by default - causing busted vertices)"),
	ECVF_Default);


namespace UE::SkeletalRender::Settings
{
	float GetMorphTargetMaxBlendWeight()
	{
		static const IConsoleVariable* MorphTargetMaxBlendWeightCVar = IConsoleManager::Get().FindConsoleVariable(TEXT("r.MorphTarget.MaxBlendWeight"));

		return (MorphTargetMaxBlendWeightCVar != nullptr) ? MorphTargetMaxBlendWeightCVar->GetFloat() : GetDefault<URendererSettings>()->MorphTargetMaxBlendWeight;
	}
}

#if RHI_RAYTRACING
static bool IsSkeletalMeshRayTracingSupported()
{
	static const auto CVar = IConsoleManager::Get().FindConsoleVariable(TEXT("r.RayTracing.Geometry.SupportSkeletalMeshes"));
	static const bool SupportSkeletalMeshes = CVar->GetInt() != 0;
	return SupportSkeletalMeshes;
}
#endif // RHI_RAYTRACING

static TAutoConsoleVariable<bool> CVarSkeletalMeshClothBlendEnabled(TEXT("r.SkeletalMeshClothBlend.Enabled"), true, TEXT("Enable the use of the cloth blend weight value set by the skeletal mesh component. When disabled all cloth blend weight will become 0."));

/*-----------------------------------------------------------------------------
FSkeletalMeshObject
-----------------------------------------------------------------------------*/
FSkeletalMeshObject::FSkeletalMeshObject(
	const USkinnedMeshComponent* InMeshComponent,
	FSkeletalMeshRenderData* InSkelMeshRenderData,
	ERHIFeatureLevel::Type InFeatureLevel)
	: FSkeletalMeshObject(FSkinnedMeshSceneProxyDesc(InMeshComponent), InSkelMeshRenderData, InFeatureLevel)
{}

FSkeletalMeshObject::FSkeletalMeshObject(const FSkinnedMeshSceneProxyDesc& InMeshDesc, FSkeletalMeshRenderData* InSkelMeshRenderData, ERHIFeatureLevel::Type InFeatureLevel)
:	MinDesiredLODLevel(FMath::Max<int32>(InMeshDesc.GetPredictedLODLevel(), InSkelMeshRenderData->CurrentFirstLODIdx))
,	MinLODLevel(InMeshDesc.GetMinLODLevel())
,	MaxDistanceFactor(0.f)
,	WorkingMinDesiredLODLevel(MinDesiredLODLevel)
,	WorkingMaxDistanceFactor(0.f)
,   bHasBeenUpdatedAtLeastOnce(false)
#if RHI_RAYTRACING
, bSupportRayTracing(IsSkeletalMeshRayTracingSupported() && InMeshDesc.GetSkinnedAsset()->GetSupportRayTracing())
, bHiddenMaterialVisibilityDirtyForRayTracing(false)
, RayTracingMinLOD(InMeshDesc.GetSkinnedAsset()->GetRayTracingMinLOD())
#endif
#if !UE_BUILD_SHIPPING
, DebugName(InMeshDesc.GetSkinnedAsset()->GetFName())
#endif // !UE_BUILD_SHIPPING
#if WITH_EDITORONLY_DATA
,   SectionIndexPreview(InMeshDesc.GetSectionPreview())
,   MaterialIndexPreview(InMeshDesc.GetMaterialPreview())
,	SelectedEditorSection(InMeshDesc.GetSelectedEditorSection())
,	SelectedEditorMaterial(InMeshDesc.GetSelectedEditorMaterial())
#endif	
,	SkeletalMeshRenderData(InSkelMeshRenderData)
,	SkinCacheEntry(nullptr)
,	SkinCacheEntryForRayTracing(nullptr)
,	LastFrameNumber(0)
,	bUsePerBoneMotionBlur(InMeshDesc.bPerBoneMotionBlur)
,	StatId(InMeshDesc.GetSkinnedAsset()->GetStatID(true))
,	FeatureLevel(InFeatureLevel)
,	ComponentId(InMeshDesc.GetPrimitiveSceneId().PrimIDValue)
,	WorldScale(InMeshDesc.GetComponentScale())
#if RHI_ENABLE_RESOURCE_INFO
,	AssetPathName(InMeshDesc.GetSkinnedAsset()->GetPathName())
#endif
{
	check(SkeletalMeshRenderData);

#if WITH_EDITORONLY_DATA
	if ( !GIsEditor )
	{
		SectionIndexPreview = -1;
		MaterialIndexPreview = -1;
	}
#endif // #if WITH_EDITORONLY_DATA

	// We want to restore the most recent value of the MaxDistanceFactor the SkeletalMeshComponent
	// cached, which will be 0.0 when first created, and a valid, updated value when recreating
	// this mesh object (e.g. during a component reregister), avoiding issues with a transient
	// assignment of 0.0 and then back to an updated value the next frame.
	MaxDistanceFactor = InMeshDesc.MaxDistanceFactor;
	WorkingMaxDistanceFactor = MaxDistanceFactor;

	InitLODInfos(InMeshDesc.LODInfo, InMeshDesc.SkinnedAsset);
}

FSkeletalMeshObject::~FSkeletalMeshObject()
{
}

void FSkeletalMeshObject::UpdateMinDesiredLODLevel(const FSceneView* View, const FBoxSphereBounds& Bounds)
{
	check(View);
	check(View->Family);

	// Thumbnail rendering doesn't contribute to MinDesiredLODLevel calculation
	if ((View->Family->bThumbnailRendering || !View->Family->GetIsInFocus()))
	{
		return;
	}

	static const auto* SkeletalMeshLODRadiusScale = IConsoleManager::Get().FindTConsoleVariableDataFloat(TEXT("r.SkeletalMeshLODRadiusScale"));
	float LODScale = FMath::Clamp(SkeletalMeshLODRadiusScale->GetValueOnRenderThread(), 0.25f, 1.0f);

	const float ScreenRadiusSquared = ComputeBoundsScreenRadiusSquared(Bounds.Origin, Bounds.SphereRadius, *View) * LODScale * LODScale;
	const uint32 FrameNumber = View->Family->FrameNumber;

	checkf(LODInfo.Num() == SkeletalMeshRenderData->LODRenderData.Num(), TEXT("Mismatched LOD arrays. LODInfo.Num() = %d, SkeletalMeshRenderData->LODRenderData.Num() = %d"), LODInfo.Num(), SkeletalMeshRenderData->LODRenderData.Num());

	// Need the current LOD
	const int32 CurrentLODLevel = GetLOD();

	int32 NewLODLevel = 0;

	// Look for a lower LOD if the EngineShowFlags is enabled
	if (View->Family->EngineShowFlags.LOD)
	{
		// Iterate from worst to best LOD
		for (int32 LODLevel = SkeletalMeshRenderData->LODRenderData.Num() - 1; LODLevel > 0; LODLevel--)
		{
			// Get ScreenSize for this LOD
			float ScreenSize = LODInfo[LODLevel].ScreenSize.GetValue();

			// If we are considering shifting to a better (lower) LOD, bias with hysteresis.
			if (LODLevel  <= CurrentLODLevel)
			{
				ScreenSize += LODInfo[LODLevel].LODHysteresis;
			}

			// If have passed this boundary, use this LOD
			if (FMath::Square(ScreenSize * 0.5f) > ScreenRadiusSquared)
			{
				NewLODLevel = LODLevel;
				break;
			}
		}
	}

	// When rendering multiple views we need to guard the assignment with a mutex since relevance can occur in parallel.
	const bool bMultiView = View->Family->AllViews.Num() > 1;
	if (bMultiView)
	{
		DesiredLODLevelMutex.Lock();
	}

	if (!LastFrameNumber)
	{
		// We don't have last frame value on the first call to FSkeletalMeshObject::UpdateMinDesiredLODLevel so
		// just reuse current frame value. Otherwise, MinDesiredLODLevel may get stale value in the update code below
		WorkingMinDesiredLODLevel = NewLODLevel;
	}

	// Different path for first-time vs subsequent-times in this function (ie splitscreen)
	if(FrameNumber != LastFrameNumber)
	{
		// Copy last frames value to the version that will be read by game thread
		MaxDistanceFactor = WorkingMaxDistanceFactor;
		MinDesiredLODLevel = WorkingMinDesiredLODLevel;
		LastFrameNumber = FrameNumber;

		WorkingMaxDistanceFactor = ScreenRadiusSquared;
		WorkingMinDesiredLODLevel = NewLODLevel;
	}
	else
	{
		WorkingMaxDistanceFactor = FMath::Max(WorkingMaxDistanceFactor, ScreenRadiusSquared);
		WorkingMinDesiredLODLevel = FMath::Min(WorkingMinDesiredLODLevel, NewLODLevel);
	}

	if (bMultiView)
	{
		DesiredLODLevelMutex.Unlock();
	}
}

/**
 * List of chunks to be rendered based on instance weight usage. Full swap of weights will render with its own chunks.
 * @return Chunks to iterate over for rendering
 */
const TArray<FSkelMeshRenderSection>& FSkeletalMeshObject::GetRenderSections(int32 InLODIndex) const
{
	const FSkeletalMeshLODRenderData& LOD = SkeletalMeshRenderData->LODRenderData[InLODIndex];
	return LOD.RenderSections;
}

FColor FSkeletalMeshObject::GetSkinCacheVisualizationDebugColor(const FName& GPUSkinCacheVisualizationMode, uint32 SectionIndex) const
{
	FGPUSkinCacheEntry* RTEntry = nullptr;
#if RHI_RAYTRACING
	RTEntry = GetSkinCacheEntryForRayTracing();
#endif
	return FGPUSkinCache::GetVisualizationDebugColor(GPUSkinCacheVisualizationMode, SkinCacheEntry, RTEntry, SectionIndex);

}

/**
 * Update the hidden material section flags for an LOD entry
 *
 * @param InLODIndex - LOD entry to update hidden material flags for
 * @param HiddenMaterials - array of hidden material sections
 */
void FSkeletalMeshObject::SetHiddenMaterials(int32 InLODIndex,const TArray<bool>& HiddenMaterials)
{
	check(LODInfo.IsValidIndex(InLODIndex));
#if RHI_RAYTRACING
	bHiddenMaterialVisibilityDirtyForRayTracing = true;
#endif
	LODInfo[InLODIndex].HiddenMaterials = HiddenMaterials;
}

/**
 * Determine if the material section entry for an LOD is hidden or not
 *
 * @param InLODIndex - LOD entry to get hidden material flags for
 * @param MaterialIdx - index of the material section to check
 */
bool FSkeletalMeshObject::IsMaterialHidden(int32 InLODIndex,int32 MaterialIdx) const
{
	check(LODInfo.IsValidIndex(InLODIndex));
	return LODInfo[InLODIndex].HiddenMaterials.IsValidIndex(MaterialIdx) && LODInfo[InLODIndex].HiddenMaterials[MaterialIdx];
}
/**
 * Initialize the array of LODInfo based on the settings of the current skel mesh component
 */
void FSkeletalMeshObject::InitLODInfos(const TArrayView<const FSkelMeshComponentLODInfo> InComponentLODInfo, const USkinnedAsset* InSkinnedAsset)
{
	const int32 LODCount = InSkinnedAsset->GetLODNum();
	
	LODInfo.Reset(LODCount);
	for (int32 Idx=0; Idx < LODCount; Idx++)
	{
		const FSkeletalMeshLODInfo& MeshLODInfo = *InSkinnedAsset->GetLODInfo(Idx);
		FSkelMeshObjectLODInfo& MeshObjectLODInfo = LODInfo.AddDefaulted_GetRef();

		MeshObjectLODInfo.ScreenSize = MeshLODInfo.ScreenSize;
		MeshObjectLODInfo.LODHysteresis = MeshLODInfo.LODHysteresis;
		
		if (InComponentLODInfo.IsValidIndex(Idx))
		{
			const FSkelMeshComponentLODInfo &Info = InComponentLODInfo[Idx];
			MeshObjectLODInfo.HiddenMaterials = Info.HiddenMaterials;
		}

#if RHI_ENABLE_RESOURCE_INFO
		MeshObjectLODInfo.AssetPathName = FName(FString::Printf(TEXT("%s [LOD%d]"), *AssetPathName.ToString(), Idx));
#endif
	}
}

float FSkeletalMeshObject::GetScreenSize(int32 LODIndex) const
{
	if (LODInfo.IsValidIndex(LODIndex))
	{
		return LODInfo[LODIndex].ScreenSize.GetValue();
	}
	return 0.f;
}

FName FSkeletalMeshObject::GetAssetPathName(int32 LODIndex) const
{
#if RHI_ENABLE_RESOURCE_INFO
	if (LODIndex > -1)
	{
		return LODInfo[LODIndex].AssetPathName;
	}
	else
	{
		return AssetPathName;
	}
#else
	return NAME_None;
#endif
}

FSkinWeightVertexBuffer* FSkeletalMeshObject::GetSkinWeightVertexBuffer(FSkeletalMeshLODRenderData& LODData, const FSkelMeshComponentLODInfo* CompLODInfo)
{
	if (CompLODInfo)
	{
		if (CompLODInfo->OverrideSkinWeights && CompLODInfo->OverrideSkinWeights->GetNumVertices() == LODData.StaticVertexBuffers.PositionVertexBuffer.GetNumVertices())
		{
			return CompLODInfo->OverrideSkinWeights;
		}
		
		FSkinWeightProfileStack ActiveProfileStack;
		const bool bRenderSideProfiles = IsSkinnedMeshSkinWeightProfilesOnRenderThread();
		
		if (bRenderSideProfiles 
			&& CompLODInfo->GetActiveSkinWeightProfileStack(ActiveProfileStack))
		{
			// If buffer is null, it means it has not been created yet and probably
			// GPU Readback has been enqueued. The default override buffer will be 
			// returned. Once the buffer is ready, skinned mesh component will update 
			// its skin weight buffer accordingly. 
			FSkinWeightVertexBuffer* OverrideProfileSkinWeights = LODData.GetSkinWeightProfileVertexBuffer(ActiveProfileStack);
			if (OverrideProfileSkinWeights)
			{
				check(OverrideProfileSkinWeights->GetNumVertices() == LODData.StaticVertexBuffers.PositionVertexBuffer.GetNumVertices());
				check(LODData.SkinWeightVertexBuffer.GetMaxBoneInfluences() == OverrideProfileSkinWeights->GetMaxBoneInfluences());
				return OverrideProfileSkinWeights;
			}
		}
		
		if (!bRenderSideProfiles && CompLODInfo->OverrideProfileSkinWeights &&
				CompLODInfo->OverrideProfileSkinWeights->GetNumVertices() == LODData.StaticVertexBuffers.PositionVertexBuffer.GetNumVertices())
		{
			check(LODData.SkinWeightVertexBuffer.GetMaxBoneInfluences() == CompLODInfo->OverrideProfileSkinWeights->GetMaxBoneInfluences());
			return CompLODInfo->OverrideProfileSkinWeights;
		}
	}
	
	return LODData.GetSkinWeightVertexBuffer();
}

FColorVertexBuffer* FSkeletalMeshObject::GetColorVertexBuffer(FSkeletalMeshLODRenderData& LODData, const FSkelMeshComponentLODInfo* CompLODInfo)
{
	// If we have a vertex color override buffer (and it's the right size) use it
	if (CompLODInfo && CompLODInfo->OverrideVertexColors &&
		CompLODInfo->OverrideVertexColors->GetNumVertices() == LODData.StaticVertexBuffers.PositionVertexBuffer.GetNumVertices())
	{
		return CompLODInfo->OverrideVertexColors;
	}
	else
	{
		return &LODData.StaticVertexBuffers.ColorVertexBuffer;
	}
}

void FSkeletalMeshObject::Update(int32 LODIndex, const USkinnedMeshComponent* InMeshComponent, const FMorphTargetWeightMap& InActiveMorphTargets, const TArray<float>& MorphTargetWeights, EPreviousBoneTransformUpdateMode PreviousBoneTransformUpdateMode, const FExternalMorphWeightData& InExternalMorphWeightData)
{
	return Update(LODIndex, FSkinnedMeshSceneProxyDynamicData(InMeshComponent), InMeshComponent->GetSceneProxy(), InMeshComponent->GetSkinnedAsset(), InActiveMorphTargets, MorphTargetWeights, PreviousBoneTransformUpdateMode, InExternalMorphWeightData);
}

void FSkeletalMeshObject::UpdateSkinWeightBuffer(const USkinnedMeshComponent* InMeshComponent)
{
	UpdateSkinWeightBuffer(InMeshComponent->LODInfo);
}

/*-----------------------------------------------------------------------------
Global functions
-----------------------------------------------------------------------------*/

void UpdateRefToLocalMatricesInner(
	const TArrayView<const FTransform> ComponentTransform,
	const TArrayView<const uint8> BoneVisibilityStates,
	TArray<FMatrix44f>& ReferenceToLocal,
	const FSkinnedMeshSceneProxyDynamicData& DynamicData,
	const USkinnedAsset* SkinnedAsset,
	const FSkeletalMeshRenderData* InSkeletalMeshRenderData,
	int32 LODIndex,
	const TArray<FBoneIndexType>* ExtraRequiredBoneIndices,
	TArray<FTransform>* LeaderBoneMappedComponentSpaceTransform)
{
	if (!InSkeletalMeshRenderData->LODRenderData.IsValidIndex(LODIndex))
	{
		UE_LOG(LogSkeletalMesh, Error,
			TEXT("Mesh %s : Invalid LODIndex [count %d, index %d], streaming[Ready(%d), F(%d), P(%d)], \
			ExtraRequiredBoneIndices is (%d), and total number is (%d)"), *GetNameSafe(SkinnedAsset),
			InSkeletalMeshRenderData->LODRenderData.Num(), LODIndex, InSkeletalMeshRenderData->bReadyForStreaming,
			InSkeletalMeshRenderData->CurrentFirstLODIdx, InSkeletalMeshRenderData->PendingFirstLODIdx,
			(ExtraRequiredBoneIndices) ? 1 : 0, (ExtraRequiredBoneIndices) ? ExtraRequiredBoneIndices->Num() : 0);

		for (int32 Index = 0; Index < ReferenceToLocal.Num(); ++Index)
		{
			ReferenceToLocal[Index] = FMatrix44f::Identity;
		}

		return;
	}

	const FReferenceSkeleton& RefSkeleton                  = SkinnedAsset->GetRefSkeleton();
	const TArray<FMatrix44f>* RefBasesInvMatrix            = &SkinnedAsset->GetRefBasesInvMatrix();

	// Check if there is an override (and it's the right size)
	if (DynamicData.GetRefPoseOverride() && DynamicData.GetRefPoseOverride()->RefBasesInvMatrix.Num() == RefBasesInvMatrix->Num())
	{
		RefBasesInvMatrix = &DynamicData.GetRefPoseOverride()->RefBasesInvMatrix;
	}

	const int32 NumRefBasesInvMatrix = RefBasesInvMatrix->Num();

	check(NumRefBasesInvMatrix != 0);

	TBitArray<FConcurrentLinearBitArrayAllocator> BitsToReset;
	BitsToReset.Init(true, NumRefBasesInvMatrix);

	const auto SetValidReferenceToLocal = [&] (int32 Index, const FMatrix44f& Matrix)
	{
		ReferenceToLocal[Index] = Matrix;
		BitsToReset[Index] = false;
	};

	const auto IsValidReferenceToLocal = [&] (int32 Index)
	{
		return !BitsToReset[Index];
	};

	ReferenceToLocal.Reset();

	if (ReferenceToLocal.Max() < NumRefBasesInvMatrix)
	{
		ReferenceToLocal.Reserve(NumRefBasesInvMatrix);
	}

	ReferenceToLocal.AddUninitialized(NumRefBasesInvMatrix);

	const bool bIsLeaderCompValid                          = DynamicData.HasLeaderPoseComponent() && DynamicData.GetLeaderBoneMap().Num() == SkinnedAsset->GetRefSkeleton().GetNum();
	const TArrayView<const int32>* LeaderBoneMap           = bIsLeaderCompValid ? &DynamicData.GetLeaderBoneMap() : nullptr;

	const FSkeletalMeshLODRenderData& LOD = InSkeletalMeshRenderData->LODRenderData[LODIndex];

	const TArray<FBoneIndexType>* RequiredBoneSets[3] = { &LOD.ActiveBoneIndices, ExtraRequiredBoneIndices, NULL };

	const bool bBoneVisibilityStatesValid = BoneVisibilityStates.Num() == ComponentTransform.Num();
	const bool bCheckForValidMeshPoseOnLeader = UE::Anim::bUseSafeMeshPoseIndices && bIsLeaderCompValid && !DynamicData.GetLeaderValidMeshPoseTransforms()->IsEmpty();

	for (int32 RequiredBoneSetIndex = 0; RequiredBoneSets[RequiredBoneSetIndex] != NULL; RequiredBoneSetIndex++)
	{
		const TArray<FBoneIndexType>& RequiredBoneIndices = *RequiredBoneSets[RequiredBoneSetIndex];

		// Get the index of the bone in this skeleton, and loop up in table to find index in parent component mesh.
		const int32 NumRequiredBoneIndices = RequiredBoneIndices.Num();
		for (int32 BoneIndex = 0; BoneIndex < NumRequiredBoneIndices; BoneIndex++)
		{
			const int32 ThisBoneIndex = RequiredBoneIndices[BoneIndex];

			if (!RefBasesInvMatrix->IsValidIndex(ThisBoneIndex))
			{
				continue;
			}

			// If we have Leader pose component, we use LeaderBoneMap to figure out the mapping
			if( bIsLeaderCompValid )
			{
				// If valid, use matrix from parent component.
				const int32 LeaderBoneIndex = (*LeaderBoneMap)[ThisBoneIndex];
				const TBitArray<>& LeaderValidMeshPoseTransforms = *(DynamicData.GetLeaderValidMeshPoseTransforms());

				if (ComponentTransform.IsValidIndex(LeaderBoneIndex) && (!bCheckForValidMeshPoseOnLeader || LeaderValidMeshPoseTransforms[LeaderBoneIndex]))
				{
					const int32 ParentIndex = RefSkeleton.GetParentIndex(ThisBoneIndex);
					bool bNeedToHideBone = BoneVisibilityStates[LeaderBoneIndex] != BVS_Visible;
					if (bNeedToHideBone && ParentIndex != INDEX_NONE)
					{
						SetValidReferenceToLocal(ThisBoneIndex, ReferenceToLocal[ParentIndex].ApplyScale(0.f));
					}
					else
					{
						checkSlow(ComponentTransform[LeaderBoneIndex].IsRotationNormalized());
						SetValidReferenceToLocal(ThisBoneIndex, (FMatrix44f)ComponentTransform[LeaderBoneIndex].ToMatrixWithScale());
					}
				}
				else
				{
					const int32 ParentIndex = RefSkeleton.GetParentIndex(ThisBoneIndex);
					const FMatrix44f RefLocalPose = (FMatrix44f)RefSkeleton.GetRefBonePose()[ThisBoneIndex].ToMatrixWithScale();
					if (ParentIndex != INDEX_NONE)
					{
						SetValidReferenceToLocal(ThisBoneIndex, RefLocalPose * ReferenceToLocal[ParentIndex]);
					}
					else
					{
						SetValidReferenceToLocal(ThisBoneIndex, RefLocalPose);
					}
				}
			}
			else
			{
				if (ComponentTransform.IsValidIndex(ThisBoneIndex))
				{
					if (bBoneVisibilityStatesValid)
					{
						// If we can't find this bone in the parent, we just use the reference pose.
						const int32 ParentIndex = RefSkeleton.GetParentIndex(ThisBoneIndex);
						bool bNeedToHideBone = BoneVisibilityStates[ThisBoneIndex] != BVS_Visible;
						if (bNeedToHideBone && ParentIndex != INDEX_NONE)
						{
							SetValidReferenceToLocal(ThisBoneIndex, ReferenceToLocal[ParentIndex].ApplyScale(0.f));
						}
						else
						{
							checkSlow(ComponentTransform[ThisBoneIndex].IsRotationNormalized());
							SetValidReferenceToLocal(ThisBoneIndex, (FMatrix44f)ComponentTransform[ThisBoneIndex].ToMatrixWithScale());
						}
					}
					else
					{
						checkSlow(ComponentTransform[ThisBoneIndex].IsRotationNormalized());
						SetValidReferenceToLocal(ThisBoneIndex, (FMatrix44f)ComponentTransform[ThisBoneIndex].ToMatrixWithScale());
					}
				}
			}
		}
	}

	for (TConstSetBitIterator BitIt(BitsToReset); BitIt; ++BitIt)
	{
		ReferenceToLocal[BitIt.GetIndex()] = FMatrix44f::Identity;
	}

	const int32 NumReferenceToLocal = ReferenceToLocal.Num();

	if (LeaderBoneMappedComponentSpaceTransform && bIsLeaderCompValid)
	{
		LeaderBoneMappedComponentSpaceTransform->SetNumUninitialized(NumReferenceToLocal);
		for (int32 ThisBoneIndex = 0; ThisBoneIndex < NumReferenceToLocal; ++ThisBoneIndex)
		{
			(*LeaderBoneMappedComponentSpaceTransform)[ThisBoneIndex] = FTransform((FMatrix)ReferenceToLocal[ThisBoneIndex]);
		}
	}

	for (int32 ThisBoneIndex = 0; ThisBoneIndex < NumReferenceToLocal; ++ThisBoneIndex)
	{
		if (!BitsToReset[ThisBoneIndex])
		{
			VectorMatrixMultiply(&ReferenceToLocal[ThisBoneIndex], &(*RefBasesInvMatrix)[ThisBoneIndex], &ReferenceToLocal[ThisBoneIndex]);
		}
	}
}
/**
 * Utility function that fills in the array of ref-pose to local-space matrices using 
 * the mesh component's updated space bases
 * @param	ReferenceToLocal - matrices to update
 * @param	SkeletalMeshComponent - mesh primitive with updated bone matrices
 * @param	LODIndex - each LOD has its own mapping of bones to update
 * @param	ExtraRequiredBoneIndices - any extra bones apart from those active in the LOD that we'd like to update
 * @param	LeaderBoneMappedComponentSpaceTransform - optional output of follower component space transforms.
 */
void UpdateRefToLocalMatrices( TArray<FMatrix44f>& ReferenceToLocal, const FSkinnedMeshSceneProxyDynamicData& InDynamicData, const USkinnedAsset* InSkinnedAsset, const FSkeletalMeshRenderData* InSkeletalMeshRenderData, 
	int32 LODIndex, const TArray<FBoneIndexType>* ExtraRequiredBoneIndices, TArray<FTransform>* LeaderBoneMappedComponentSpaceTransform)
{
	const TArrayView<const FTransform> ComponentTransforms = InDynamicData.GetComponentSpaceTransforms();
	const TArrayView<const uint8> BoneVisibilityStates     = InDynamicData.GetBoneVisibilityStates();

	UpdateRefToLocalMatricesInner(ComponentTransforms, BoneVisibilityStates, ReferenceToLocal, InDynamicData, InSkinnedAsset, InSkeletalMeshRenderData, LODIndex, ExtraRequiredBoneIndices, LeaderBoneMappedComponentSpaceTransform);
}

void UpdatePreviousRefToLocalMatrices(TArray<FMatrix44f>& ReferenceToLocal, const FSkinnedMeshSceneProxyDynamicData& InDynamicData, const USkinnedAsset* InSkinnedAsset, const FSkeletalMeshRenderData* InSkeletalMeshRenderData, int32 LODIndex, const TArray<FBoneIndexType>* ExtraRequiredBoneIndices)
{
	const TArrayView<const FTransform> ComponentTransforms = InDynamicData.GetPreviousComponentTransformsArray();
	const TArrayView<const uint8> BoneVisibilityStates     = InDynamicData.GetPreviousBoneVisibilityStates();

	UpdateRefToLocalMatricesInner(ComponentTransforms, BoneVisibilityStates, ReferenceToLocal, InDynamicData, InSkinnedAsset, InSkeletalMeshRenderData, LODIndex, ExtraRequiredBoneIndices, nullptr);
}

void UpdateBonesRemovedByLOD(
	FSkeletalMeshObject* MeshObject,
	TArray<FMatrix44f>& PoseBuffer,
	const FSkinnedMeshSceneProxyDynamicData& InDynamicData,
	const USkinnedAsset* SkinnedAsset,
	int32 LODIndex,
	bool bUsePreviousTransforms)
{
	// When the animation system removes bones at higher LODs, the pose in USkinnedMeshComponent::GetComponentSpaceTransforms()
	// will leave the LOD'd bone transforms at their last updated position/rotation. This is not a problem for standard GPU skinning
	// because the actual weight for those bones is pushed up the hierarchy onto the next non-LOD'd parent; making the transform irrelevant.
	//
	// But Nanite skinning uses LOD-0 weights (dynamically interpolating for higher-LOD clusters), and the ray tracing LOD may differ
	// from the main rendering LOD. In either case, the "frozen" bone transforms affect the skin and must be corrected.
	//
	// This function performs an FK update of the frozen branch of transforms to put them back in their ref pose relative position.

	TConstArrayView<FBoneReference> BonesToRemove = MeshObject->GetCachedBonesToRemove(SkinnedAsset, LODIndex);
	if (BonesToRemove.IsEmpty())
	{
		return;
	}

	TArrayView<const FTransform> ComponentSpacePose = bUsePreviousTransforms
		? InDynamicData.GetPreviousComponentTransformsArray()
		: InDynamicData.GetComponentSpaceTransforms();

	if (ComponentSpacePose.IsEmpty())
	{
		return;
	}

	const TArray<FMatrix44f>* RefBasesInvMatrix = &SkinnedAsset->GetRefBasesInvMatrix();
	const FReferenceSkeleton& RefSkeleton = SkinnedAsset->GetRefSkeleton();
	const int32 NumBones = RefSkeleton.GetRawBoneNum();

	if (PoseBuffer.Num() < NumBones || RefBasesInvMatrix->Num() < NumBones)
	{
		return;
	}

	TArray<int32> AllChildrenBones;
	for (const FBoneReference& RemovedBone : BonesToRemove)
	{
		AllChildrenBones.Reset();
		const int32 BoneIndex = RefSkeleton.FindRawBoneIndex(RemovedBone.BoneName);
		if (BoneIndex == INDEX_NONE)
		{
			continue;
		}
		AllChildrenBones.Add(BoneIndex);
		RefSkeleton.GetRawChildrenIndicesRecursiveCached(BoneIndex, AllChildrenBones);

		// First pass: generate component space transforms from ref pose
		for (int32 ChildIndex = 0; ChildIndex < AllChildrenBones.Num(); ++ChildIndex)
		{
			const int32 ChildBoneIndex = AllChildrenBones[ChildIndex];
			const int32 ParentIndex = RefSkeleton.GetParentIndex(ChildBoneIndex);

			FMatrix44f ParentComponentTransform;
			if (ParentIndex == INDEX_NONE)
			{
				ParentComponentTransform = FMatrix44f::Identity;
			}
			else if (ChildIndex == 0)
			{
				ParentComponentTransform = ComponentSpacePose.IsValidIndex(ParentIndex)
					? static_cast<FMatrix44f>(ComponentSpacePose[ParentIndex].ToMatrixWithScale())
					: FMatrix44f::Identity;
			}
			else
			{
				ParentComponentTransform = PoseBuffer[ParentIndex];
			}

			const FMatrix44f RefLocalTransform = static_cast<FMatrix44f>(RefSkeleton.GetRefBonePose()[ChildBoneIndex].ToMatrixWithScale());
			PoseBuffer[ChildBoneIndex] = RefLocalTransform * ParentComponentTransform;
		}

		// Second pass: make relative to ref pose
		for (const int32 ChildBoneIndex : AllChildrenBones)
		{
			PoseBuffer[ChildBoneIndex] = (*RefBasesInvMatrix)[ChildBoneIndex] * PoseBuffer[ChildBoneIndex];
		}
	}
}

bool IsSkeletalMeshClothBlendEnabled()
{
	return CVarSkeletalMeshClothBlendEnabled.GetValueOnAnyThread();
}

bool IsSkinnedMeshSkinWeightProfilesOnRenderThread()
{
	return CVarSkinnedMeshSkinWeightProfilesOnRenderThread.GetValueOnAnyThread();
}
