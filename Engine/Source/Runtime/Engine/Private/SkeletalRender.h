// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	SkeletalRender.h: Definitions and inline code for rendering SkeletalMeshComponet
=============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "BoneIndices.h"

class FSkeletalMeshObject;
class USkinnedMeshComponent;
class FSkeletalMeshRenderData;
class USkinnedAsset;
struct FSkinnedMeshSceneProxyDynamicData;

// smallest blend weight for vertex anims
extern const float MinMorphTargetBlendWeight;
// largest blend weight for vertex anims
UE_DEPRECATED(5.4, "Please, use UE::SkeletalRender::Settings::GetMorphTargetMaxBlendWeight");
extern const float MaxMorphTargetBlendWeight;


/**
* Utility function that fills in the array of ref-pose to local-space matrices using 
* the mesh component's updated space bases
* @param	ReferenceToLocal - matrices to update
* @param	InDynamicData - mesh primitive with updated bone matrices
* @param	InSkinnedAsset - asset used to initialize mesh primitive
* @param	InSkeletalMeshRenderData - resource for which to compute RefToLocal matrices
* @param	LODIndex - each LOD has its own mapping of bones to update
* @param	ExtraRequiredBoneIndices - any extra bones apart from those active in the LOD that we'd like to update
* @param	LeaderBoneMappedComponentSpaceTransform - optional output of follower component space transforms.
*/
ENGINE_API void UpdateRefToLocalMatrices( TArray<FMatrix44f>& ReferenceToLocal, const FSkinnedMeshSceneProxyDynamicData& InDynamicData, const USkinnedAsset* InSkinnedAsset, const FSkeletalMeshRenderData* InSkeletalMeshRenderData, int32 LODIndex, const TArray<FBoneIndexType>* ExtraRequiredBoneIndices = nullptr, TArray<FTransform>* LeaderBoneMappedComponentSpaceTransform = nullptr );

/**
* Utility function that fills in the array of ref-pose to local-space matrices using 
* the mesh component's updated previous space bases
* @param	ReferenceToLocal - matrices to update
* @param	InDynamicData - mesh primitive with updated bone matrices
* @param	InSkinnedAsset - asset used to initialize mesh primitive
* @param	InSkeletalMeshRenderData - resource for which to compute RefToLocal matrices
* @param	LODIndex - each LOD has its own mapping of bones to update
* @param	ExtraRequiredBoneIndices - any extra bones apart from those active in the LOD that we'd like to update
*/
ENGINE_API void UpdatePreviousRefToLocalMatrices(TArray<FMatrix44f>& ReferenceToLocal, const FSkinnedMeshSceneProxyDynamicData& InDynamicData,  const USkinnedAsset* InSkinnedAsset, const FSkeletalMeshRenderData* InSkeletalMeshRenderData, int32 LODIndex, const TArray<FBoneIndexType>* ExtraRequiredBoneIndices = NULL);

/**
* Forward kinematic update of bones removed at a given LOD level.
* Fixes frozen transforms in the pose buffer for bones that were LOD'd out.
* @param	MeshObject - mesh object that caches the bones-to-remove list
* @param	PoseBuffer - reference-to-local matrices to fix up
* @param	InDynamicData - dynamic data providing component space transforms
* @param	SkinnedAsset - asset providing ref skeleton and inverse ref pose
* @param	LODIndex - LOD level whose removed bones should be fixed
* @param	bUsePreviousTransforms - if true, reads previous component space transforms instead of current
*/
ENGINE_API void UpdateBonesRemovedByLOD(FSkeletalMeshObject* MeshObject, TArray<FMatrix44f>& PoseBuffer, const FSkinnedMeshSceneProxyDynamicData& InDynamicData, const USkinnedAsset* SkinnedAsset, int32 LODIndex, bool bUsePreviousTransforms);

extern ENGINE_API const VectorRegister		VECTOR_0001;

/**
* Apply scale/bias to packed normal byte values and store result in register
* Only first 3 components are copied. W component is always 0
* 
* @param PackedNormal - source vector packed with byte components
* @return vector register with unpacked float values
*/
static FORCEINLINE VectorRegister Unpack3( const uint32 *PackedNormal )
{
	return VectorMultiply(VectorLoadSignedByte4(PackedNormal), VectorSetFloat3(1.0f / 127.0f, 1.0f / 127.0f, 1.0f / 127.0f));
}

/**
* Apply scale/bias to float register values and store results in memory as packed byte values 
* Only first 3 components are copied. W component is always 0
* 
* @param Normal - source vector register with floats
* @param PackedNormal - destination vector packed with byte components
*/
static FORCEINLINE void Pack3( VectorRegister4Float Normal, uint32 *PackedNormal )
{
	Normal = VectorMultiply(Normal, VectorSetFloat3(127.0f, 127.0f, 127.0f));
	VectorStoreSignedByte4( Normal, PackedNormal );
}

static FORCEINLINE void Pack3(VectorRegister4Double Normal, uint32* PackedNormal)
{
	Normal = VectorMultiply(Normal, VectorSetFloat3(127.0f, 127.0f, 127.0f));
	VectorStoreSignedByte4(MakeVectorRegisterFloatFromDouble(Normal), PackedNormal);
}

/**
* Apply scale/bias to packed normal byte values and store result in register
* All 4 components are copied. 
* 
* @param PackedNormal - source vector packed with byte components
* @return vector register with unpacked float values
*/
static FORCEINLINE VectorRegister Unpack4( const uint32 *PackedNormal )
{
	return VectorMultiply(VectorLoadSignedByte4(PackedNormal), VectorSetFloat1(1.0f / 127.0f));
}

/**
* Apply scale/bias to float register values and store results in memory as packed byte values 
* All 4 components are copied. 
* 
* @param Normal - source vector register with floats
* @param PackedNormal - destination vector packed with byte components
*/
static FORCEINLINE void Pack4( VectorRegister4Float Normal, uint32 *PackedNormal )
{
	Normal = VectorMultiply(Normal, VectorSetFloat1(127.0f));
	VectorStoreSignedByte4( Normal, PackedNormal );
}

static FORCEINLINE void Pack4(VectorRegister4Double Normal, uint32* PackedNormal)
{
	Normal = VectorMultiply(Normal, VectorSetFloat1(127.0f));
	VectorStoreSignedByte4(MakeVectorRegisterFloatFromDouble(Normal), PackedNormal);
}

/**
* Returns true if the cloth blend weight defined in the SkeletalMeshComponent should be used by skin rendereding.
*/
bool IsSkeletalMeshClothBlendEnabled();

/** Returns true if skin weight profile management is to be done on the render thread exclusively */
ENGINE_API bool IsSkinnedMeshSkinWeightProfilesOnRenderThread();