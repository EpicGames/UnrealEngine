// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	SkeletalRenderCPUSkin.cpp: CPU skinned skeletal mesh rendering code.
=============================================================================*/

#include "SkeletalRenderCPUSkin.h"
#include "EngineStats.h"
#include "Components/SkeletalMeshComponent.h"
#include "PrimitiveSceneProxy.h"
#include "RenderUtils.h"
#include "PrimitiveDrawingUtils.h"
#include "Rendering/SkeletalMeshRenderData.h"
#include "SkeletalRender.h"
#include "RenderCore.h"
#include "SceneInterface.h"
#include "Stats/StatsTrace.h"
#include "Rendering/RenderCommandPipes.h"
#include "SkinnedMeshSceneProxyDesc.h"

#if RHI_RAYTRACING
#include "Engine/SkinnedAssetCommon.h"
#endif

struct FMorphTargetDelta;

template<typename VertexType, int32 NumberOfUVs>
static void SkinVertices(FFinalSkinVertex* DestVertex, FMatrix44f* ReferenceToLocal, int32 LODIndex, FSkeletalMeshLODRenderData& LOD, FSkinWeightVertexBuffer& WeightBuffer, const FMorphTargetWeightMap& ActiveMorphTargets, const TArray<float>& MorphTargetWeights, const TMap<int32, FClothSimulData>& ClothSimulUpdateData, float ClothBlendWeight, const FMatrix& WorldToLocal, const FVector& WorldScale);

#define INFLUENCE_0		0
#define INFLUENCE_1		1
#define INFLUENCE_2		2
#define INFLUENCE_3		3
#define INFLUENCE_4		4
#define INFLUENCE_5		5
#define INFLUENCE_6		6
#define INFLUENCE_7		7
#define INFLUENCE_8		8
#define INFLUENCE_9		9
#define INFLUENCE_10	10
#define INFLUENCE_11	11

/** 
 * Modify the vertex buffer to store bone weights in the UV coordinates for rendering 
 * @param DestVertex - already filled out vertex buffer from SkinVertices
 * @param LOD - LOD model corresponding to DestVertex 
 * @param BonesOfInterest - array of bones we want to display
 */
static void CalculateBoneWeights(FFinalSkinVertex* DestVertex, FSkeletalMeshLODRenderData& LOD, FSkinWeightVertexBuffer& WeightBuffer, TArray<int32> InBonesOfInterest);

/**
* Modify the vertex buffer to store morph target weights in the UV coordinates for rendering
* @param DestVertex - already filled out vertex buffer from SkinVertices
* @param LOD - LOD model corresponding to DestVertex
* @param MorphTargetOfInterest - array of morphtargets to draw
*/
static void CalculateMorphTargetWeights(FFinalSkinVertex* DestVertex, FSkeletalMeshLODRenderData& LOD, int LODIndex, TArray<UMorphTarget*> InMorphTargetOfInterest);

/*-----------------------------------------------------------------------------
	FSkeletalMeshObjectCPUSkin
-----------------------------------------------------------------------------*/

FSkeletalMeshObjectCPUSkin::FSkeletalMeshObjectCPUSkin(const USkinnedMeshComponent* InMeshComponent, FSkeletalMeshRenderData* InSkelMeshRenderData, ERHIFeatureLevel::Type InFeatureLevel)
	: FSkeletalMeshObjectCPUSkin(FSkinnedMeshSceneProxyDesc(InMeshComponent), InSkelMeshRenderData, InFeatureLevel) {}

FSkeletalMeshObjectCPUSkin::FSkeletalMeshObjectCPUSkin(const FSkinnedMeshSceneProxyDesc& InMeshDesc, FSkeletalMeshRenderData* InSkelMeshRenderData, ERHIFeatureLevel::Type InFeatureLevel)
	: FSkeletalMeshObject(InMeshDesc, InSkelMeshRenderData, InFeatureLevel)
,	DynamicData(NULL)
,	CachedVertexLOD(INDEX_NONE)
,	bRenderOverlayMaterial(false)
{
	// create LODs to match the base mesh
	for( int32 LODIndex=0;LODIndex < InSkelMeshRenderData->LODRenderData.Num();LODIndex++ )
	{
		new(LODs) FSkeletalMeshObjectLOD(InFeatureLevel, InSkelMeshRenderData,LODIndex);
	}

	InitResources(InMeshDesc);

	// This must be disabled since mesh caching doesn't support recreating the vertex factory on demand.
	bSupportsStaticRelevance = false;
}


FSkeletalMeshObjectCPUSkin::~FSkeletalMeshObjectCPUSkin()
{
	delete DynamicData;
}

void FSkeletalMeshObjectCPUSkin::InitResources(const FSkinnedMeshSceneProxyDesc& InMeshDesc)
{
	for( int32 LODIndex=0;LODIndex < LODs.Num();LODIndex++ )
	{
		FSkeletalMeshObjectLOD& SkelLOD = LODs[LODIndex];

		// Skip LODs that have their render data stripped
		if (SkelLOD.SkelMeshRenderData->LODRenderData[LODIndex].GetNumVertices() > 0)
		{
			const FSkelMeshComponentLODInfo* CompLODInfo = InMeshDesc.LODInfo.IsValidIndex(LODIndex) ?  &InMeshDesc.LODInfo[LODIndex] : nullptr;
			SkelLOD.InitResources(CompLODInfo);
		}
	}
}


void FSkeletalMeshObjectCPUSkin::ReleaseResources()
{
	for( int32 LODIndex=0;LODIndex < LODs.Num();LODIndex++ )
	{
		FSkeletalMeshObjectLOD& SkelLOD = LODs[LODIndex];
		SkelLOD.ReleaseResources();
	}
}


void FSkeletalMeshObjectCPUSkin::EnableOverlayRendering(bool bEnabled, const TArray<int32>* InBonesOfInterest, const TArray<UMorphTarget*>* InMorphTargetOfInterest)
{
	bRenderOverlayMaterial = bEnabled;
	
	BonesOfInterest.Reset();
	MorphTargetOfInterest.Reset();

	if (InBonesOfInterest)
	{
		BonesOfInterest.Append(*InBonesOfInterest);
	}
	else if (InMorphTargetOfInterest)
	{
		MorphTargetOfInterest.Append(*InMorphTargetOfInterest);
	}
}

void FSkeletalMeshObjectCPUSkin::Update(
	int32 LODIndex,
	const FSkinnedMeshSceneProxyDynamicData& InDynamicData,
	const FPrimitiveSceneProxy* InSceneProxy,
	const USkinnedAsset* InSkinnedAsset,
	const FMorphTargetWeightMap& InActiveMorphTargets,
	const TArray<float>& InMorphTargetWeights,
	EPreviousBoneTransformUpdateMode PreviousBoneTransformUpdateMode,
	const FExternalMorphWeightData& InExternalMorphWeightData)
{
	// create the new dynamic data for use by the rendering thread
	// this data is only deleted when another update is sent
	FDynamicSkelMeshObjectDataCPUSkin* NewDynamicData = new FDynamicSkelMeshObjectDataCPUSkin(InDynamicData, InSkinnedAsset, SkeletalMeshRenderData, LODIndex, InActiveMorphTargets, InMorphTargetWeights);

	uint64 FrameNumberToPrepare = GFrameCounter;
	uint32 RevisionNumber = 0;

	if (InSceneProxy)
	{
		RevisionNumber = InDynamicData.GetBoneTransformRevisionNumber();
	}

	// queue a call to update this data
	{
		FSkeletalMeshObjectCPUSkin* MeshObject = this;
		ENQUEUE_RENDER_COMMAND(SkelMeshObjectUpdateDataCommand)(UE::RenderCommandPipe::SkeletalMesh,
			[MeshObject, FrameNumberToPrepare, RevisionNumber, NewDynamicData](FRHICommandList& RHICmdList)
			{
				FScopeCycleCounter Context(MeshObject->GetStatId());
				MeshObject->UpdateDynamicData_RenderThread(RHICmdList, NewDynamicData, FrameNumberToPrepare, RevisionNumber);
			}
		);
	}
}

void FSkeletalMeshObjectCPUSkin::UpdateSkinWeightBuffer(const TArrayView<const FSkelMeshComponentLODInfo> InLODInfo)
{
	for (int32 LODIndex = 0; LODIndex < LODs.Num(); LODIndex++)
	{
		FSkeletalMeshObjectLOD& SkelLOD = LODs[LODIndex];

		// Skip LODs that have their render data stripped
		if (SkelLOD.SkelMeshRenderData->LODRenderData[LODIndex].GetNumVertices() > 0)
		{
			const FSkelMeshComponentLODInfo* CompLODInfo = InLODInfo.IsValidIndex(LODIndex) ? &InLODInfo[LODIndex] : nullptr;

			SkelLOD.UpdateSkinWeights(CompLODInfo);
		}
	}
}

void FSkeletalMeshObjectCPUSkin::UpdateDynamicData_RenderThread(FRHICommandList& RHICmdList, FDynamicSkelMeshObjectDataCPUSkin* InDynamicData, uint64 FrameNumberToPrepare, uint32 RevisionNumber)
{
	// we should be done with the old data at this point
	delete DynamicData;
	// update with new data
	DynamicData = InDynamicData;	
	check(DynamicData);

	// update vertices using the new data
	CacheVertices(DynamicData->LODIndex, true, RHICmdList);
}

#define SKIN_LOD_VERTICES(VertexType, NumUVs) \
{\
	switch( NumUVs )\
    {\
        case 1:\
			SkinVertices<VertexType<1>, 1>( DestVertex, ReferenceToLocal, DynamicData->LODIndex, LOD, *MeshLOD.MeshObjectWeightBuffer, DynamicData->ActiveMorphTargets, DynamicData->MorphTargetWeights, DynamicData->ClothSimulUpdateData, DynamicData->ClothBlendWeight, DynamicData->WorldToLocal, WorldScale); \
          	break;\
        case 2:\
			SkinVertices<VertexType<2>, 2>( DestVertex, ReferenceToLocal, DynamicData->LODIndex, LOD, *MeshLOD.MeshObjectWeightBuffer, DynamicData->ActiveMorphTargets, DynamicData->MorphTargetWeights, DynamicData->ClothSimulUpdateData, DynamicData->ClothBlendWeight, DynamicData->WorldToLocal, WorldScale); \
          	break;\
        case 3:\
			SkinVertices<VertexType<3>, 3>( DestVertex, ReferenceToLocal, DynamicData->LODIndex, LOD, *MeshLOD.MeshObjectWeightBuffer, DynamicData->ActiveMorphTargets, DynamicData->MorphTargetWeights, DynamicData->ClothSimulUpdateData, DynamicData->ClothBlendWeight, DynamicData->WorldToLocal, WorldScale); \
          	break;\
        case 4:\
			SkinVertices<VertexType<4>, 4>( DestVertex, ReferenceToLocal, DynamicData->LODIndex, LOD, *MeshLOD.MeshObjectWeightBuffer, DynamicData->ActiveMorphTargets, DynamicData->MorphTargetWeights, DynamicData->ClothSimulUpdateData, DynamicData->ClothBlendWeight, DynamicData->WorldToLocal, WorldScale); \
          	break;\
        default:\
          	checkf(false, TEXT("Invalid number of UV sets.  Must be between 1 and 4") );\
			break;\
    }\
}\
	

void FSkeletalMeshObjectCPUSkin::CacheVertices(int32 LODIndex, bool bForce, FRHICommandList& RHICmdList) const
{
	SCOPE_CYCLE_COUNTER( STAT_CPUSkinUpdateRTTime);
	check(IsInParallelRenderingThread());

	// Source skel mesh and static lod model
	FSkeletalMeshLODRenderData& LOD = SkeletalMeshRenderData->LODRenderData[LODIndex];

	// Get the destination mesh LOD.
	const FSkeletalMeshObjectLOD& MeshLOD = LODs[LODIndex];

	// only recache if lod changed
	if ( (LODIndex != CachedVertexLOD || bForce) &&
		DynamicData && 
		MeshLOD.StaticMeshVertexBuffer.IsValid() )
	{
		const FSkelMeshObjectLODInfo& MeshLODInfo = LODInfo[LODIndex];

		// bone matrices
		FMatrix44f* ReferenceToLocal = DynamicData->ReferenceToLocal.GetData();

		int32 CachedFinalVerticesNum = LOD.GetNumVertices();
		CachedFinalVertices.Empty(CachedFinalVerticesNum);
		CachedFinalVertices.AddUninitialized(CachedFinalVerticesNum);

		// final cached verts
		FFinalSkinVertex* DestVertex = CachedFinalVertices.GetData();

		if (DestVertex)
		{
			check(GIsEditor || LOD.StaticVertexBuffers.StaticMeshVertexBuffer.GetAllowCPUAccess());
			SCOPE_CYCLE_COUNTER(STAT_SkinningTime);

			// do actual skinning
			if (LOD.StaticVertexBuffers.StaticMeshVertexBuffer.GetUseFullPrecisionUVs())
			{
				SKIN_LOD_VERTICES(TGPUSkinVertexFloat32Uvs, LOD.StaticVertexBuffers.StaticMeshVertexBuffer.GetNumTexCoords());
			}
			else
			{
				SKIN_LOD_VERTICES(TGPUSkinVertexFloat16Uvs, LOD.StaticVertexBuffers.StaticMeshVertexBuffer.GetNumTexCoords());
			}

			if (bRenderOverlayMaterial)
			{
				if (MorphTargetOfInterest.Num() > 0)
				{
					//Transfer morph target weights we're interested in to the UV channels
					CalculateMorphTargetWeights(DestVertex, LOD, LODIndex, MorphTargetOfInterest);
				}
				else // default is bones of interest
					// this can go if no morphtarget is selected but enabled to render
					// but that doesn't matter since it will only draw empty overlay
				{
					//Transfer bone weights we're interested in to the UV channels
					CalculateBoneWeights(DestVertex, LOD, *MeshLOD.MeshObjectWeightBuffer, BonesOfInterest);
				}
			}
		}

		// set lod level currently cached
		CachedVertexLOD = LODIndex;

		check(LOD.GetNumVertices() == CachedFinalVertices.Num());

		for (int i = 0; i < CachedFinalVertices.Num(); i++)
		{
			MeshLOD.PositionVertexBuffer.VertexPosition(i) = CachedFinalVertices[i].Position;
			MeshLOD.StaticMeshVertexBuffer.SetVertexTangents(i, (FVector3f)CachedFinalVertices[i].TangentX.ToFVector(), CachedFinalVertices[i].GetTangentY(), (FVector3f)CachedFinalVertices[i].TangentZ.ToFVector());

			for (uint32 UVIndex = 0; UVIndex < LOD.StaticVertexBuffers.StaticMeshVertexBuffer.GetNumTexCoords(); ++UVIndex)
			{
				MeshLOD.StaticMeshVertexBuffer.SetVertexUV(i, UVIndex, FVector2f(CachedFinalVertices[i].TextureCoordinates[UVIndex].X, CachedFinalVertices[i].TextureCoordinates[UVIndex].Y));
			}
		}

		MeshLOD.PositionVertexBuffer.UpdateRHI(RHICmdList);
		MeshLOD.StaticMeshVertexBuffer.UpdateRHI(RHICmdList);

		FLocalVertexFactory::FDataType Data;
		MeshLOD.PositionVertexBuffer.BindPositionVertexBuffer(&MeshLOD.VertexFactory, Data);
		MeshLOD.StaticMeshVertexBuffer.BindTangentVertexBuffer(&MeshLOD.VertexFactory, Data);
		MeshLOD.StaticMeshVertexBuffer.BindPackedTexCoordVertexBuffer(&MeshLOD.VertexFactory, Data, MAX_TEXCOORDS);
		MeshLOD.StaticMeshVertexBuffer.BindLightMapVertexBuffer(&MeshLOD.VertexFactory, Data, 0);
		MeshLOD.MeshObjectColorBuffer->BindColorVertexBuffer(&MeshLOD.VertexFactory, Data);

		MeshLOD.VertexFactory.SetData(RHICmdList, Data);
		MeshLOD.VertexFactory.InitResource(RHICmdList);
	}
}

const FVertexFactory* FSkeletalMeshObjectCPUSkin::GetSkinVertexFactory(const FSceneView* View, int32 LODIndex, int32 ChunkIdx, ESkinVertexFactoryMode VFMode) const
{
	return GetStaticSkinVertexFactory(LODIndex, ChunkIdx, VFMode);
}

const FVertexFactory* FSkeletalMeshObjectCPUSkin::GetStaticSkinVertexFactory(int32 LODIndex, int32 ChunkIdx, ESkinVertexFactoryMode VFMode) const
{
	check( LODs.IsValidIndex(LODIndex) );
	return &LODs[LODIndex].VertexFactory;
}

/** 
 * Init rendering resources for this LOD 
 */
void FSkeletalMeshObjectCPUSkin::FSkeletalMeshObjectLOD::InitResources(const FSkelMeshComponentLODInfo* CompLODInfo)
{
	check(SkelMeshRenderData);
	check(SkelMeshRenderData->LODRenderData.IsValidIndex(LODIndex));

	// If we have a skin weight override buffer (and it's the right size) use it
	FSkeletalMeshLODRenderData& LODData = SkelMeshRenderData->LODRenderData[LODIndex];
	MeshObjectWeightBuffer = FSkeletalMeshObject::GetSkinWeightVertexBuffer(LODData, CompLODInfo);
	MeshObjectColorBuffer = FSkeletalMeshObject::GetColorVertexBuffer(LODData, CompLODInfo);

	const FStaticMeshVertexBuffer& SrcVertexBuf = LODData.StaticVertexBuffers.StaticMeshVertexBuffer;
	PositionVertexBuffer.Init(LODData.StaticVertexBuffers.PositionVertexBuffer);
	StaticMeshVertexBuffer.Init(SrcVertexBuf.GetNumVertices(), MAX_TEXCOORDS);

	for (uint32 i = 0; i < SrcVertexBuf.GetNumVertices(); i++)
	{
		StaticMeshVertexBuffer.SetVertexTangents(i, SrcVertexBuf.VertexTangentX(i), SrcVertexBuf.VertexTangentY(i), SrcVertexBuf.VertexTangentZ(i));
		StaticMeshVertexBuffer.SetVertexUV(i, 0, SrcVertexBuf.GetVertexUV(i, 0));
	}

	BeginInitResource(&PositionVertexBuffer, &UE::RenderCommandPipe::SkeletalMesh);
	BeginInitResource(&StaticMeshVertexBuffer, &UE::RenderCommandPipe::SkeletalMesh);

	FSkeletalMeshObjectCPUSkin::FSkeletalMeshObjectLOD* Self = this;
	FLocalVertexFactory* VertexFactoryPtr = &VertexFactory;
	// update vertex factory components and sync it
	ENQUEUE_RENDER_COMMAND(InitSkeletalMeshCPUSkinVertexFactory)(UE::RenderCommandPipe::SkeletalMesh,
		[VertexFactoryPtr, Self](FRHICommandListBase& RHICmdList)
		{
			FLocalVertexFactory::FDataType Data;

			Self->PositionVertexBuffer.BindPositionVertexBuffer(VertexFactoryPtr, Data);
			Self->StaticMeshVertexBuffer.BindTangentVertexBuffer(VertexFactoryPtr, Data);
			Self->StaticMeshVertexBuffer.BindPackedTexCoordVertexBuffer(VertexFactoryPtr, Data, MAX_TEXCOORDS);
			Self->StaticMeshVertexBuffer.BindLightMapVertexBuffer(VertexFactoryPtr, Data, 0);
			Self->MeshObjectColorBuffer->BindColorVertexBuffer(VertexFactoryPtr, Data);

			VertexFactoryPtr->SetData(RHICmdList, Data);
			VertexFactoryPtr->InitResource(RHICmdList);
		});

#if RHI_RAYTRACING
	if (IsRayTracingEnabled() && SkelMeshRenderData->bSupportRayTracing)
	{
		check(SkelMeshRenderData);
		check(SkelMeshRenderData->LODRenderData.IsValidIndex(LODIndex));
		FSkeletalMeshLODRenderData& LODModel = SkelMeshRenderData->LODRenderData[LODIndex];
		FBufferRHIRef VertexBufferRHI = LODModel.StaticVertexBuffers.PositionVertexBuffer.VertexBufferRHI;
		FBufferRHIRef IndexBufferRHI = LODModel.MultiSizeIndexContainer.GetIndexBuffer()->IndexBufferRHI;
		uint32 VertexBufferNumVertices = LODModel.StaticVertexBuffers.PositionVertexBuffer.GetNumVertices();
		uint32 VertexBufferStride = LODModel.StaticVertexBuffers.PositionVertexBuffer.GetStride();

		uint32 TrianglesCount = 0;
		for (int32 SectionIndex = 0; SectionIndex < LODModel.RenderSections.Num(); SectionIndex++)
		{
			const FSkelMeshRenderSection& Section = LODModel.RenderSections[SectionIndex];
			TrianglesCount += Section.NumTriangles;
		}

		TArray<FSkelMeshRenderSection>* RenderSections = &LODModel.RenderSections;
		ENQUEUE_RENDER_COMMAND(InitSkeletalRenderCPUSkinRayTracingGeometry)(UE::RenderCommandPipe::SkeletalMesh,
			[this, VertexBufferRHI, IndexBufferRHI, VertexBufferNumVertices, VertexBufferStride, TrianglesCount, RenderSections, &SourceGeometry = LODModel.SourceRayTracingGeometry](FRHICommandListBase& RHICmdList)
			{
				FRayTracingGeometryInitializer Initializer;
				static const FName DebugName("FSkeletalMeshObjectCPUSkin");
				static int32 DebugNumber = 0;
				Initializer.DebugName = FDebugName(DebugName, DebugNumber++);
				Initializer.IndexBuffer = IndexBufferRHI;
				Initializer.TotalPrimitiveCount = TrianglesCount;
				Initializer.GeometryType = RTGT_Triangles;
				Initializer.bFastBuild = true;

				TArray<FRayTracingGeometrySegment> GeometrySections;
				GeometrySections.Reserve(RenderSections->Num());

				for (const FSkelMeshRenderSection& Section : *RenderSections)
				{
					FRayTracingGeometrySegment Segment;
					Segment.VertexBuffer = VertexBufferRHI;
					Segment.VertexBufferStride = VertexBufferStride;
					Segment.VertexBufferOffset = 0;
					Segment.VertexBufferElementType = VET_Float3;
					Segment.MaxVertices = VertexBufferNumVertices;
					Segment.FirstPrimitive = Section.BaseIndex / 3;
					Segment.NumPrimitives = Section.NumTriangles;
					Segment.bEnabled = !Section.bDisabled && Section.bVisibleInRayTracing;
					GeometrySections.Add(Segment);
				}
				Initializer.Segments = GeometrySections;
				Initializer.SourceGeometry = SourceGeometry.GetRHI();

				RayTracingGeometry.SetInitializer(Initializer);
				RayTracingGeometry.InitResource(RHICmdList);
			}
		);
	}
#endif // RHI_RAYTRACING

	bResourcesInitialized = true;
}

void FSkeletalMeshObjectCPUSkin::FSkeletalMeshObjectLOD::UpdateSkinWeights(const FSkelMeshComponentLODInfo* CompLODInfo)
{
	check(SkelMeshRenderData);
	check(SkelMeshRenderData->LODRenderData.IsValidIndex(LODIndex));

	// If we have a skin weight override buffer (and it's the right size) use it
	FSkeletalMeshLODRenderData& LODData = SkelMeshRenderData->LODRenderData[LODIndex];
	MeshObjectWeightBuffer = FSkeletalMeshObject::GetSkinWeightVertexBuffer(LODData, CompLODInfo);
}

/** 
 * Release rendering resources for this LOD 
 */
void FSkeletalMeshObjectCPUSkin::FSkeletalMeshObjectLOD::ReleaseResources()
{	
	BeginReleaseResource(&VertexFactory, &UE::RenderCommandPipe::SkeletalMesh);
	BeginReleaseResource(&PositionVertexBuffer, &UE::RenderCommandPipe::SkeletalMesh);
	BeginReleaseResource(&StaticMeshVertexBuffer, &UE::RenderCommandPipe::SkeletalMesh);

#if RHI_RAYTRACING
	BeginReleaseResource(&RayTracingGeometry, &UE::RenderCommandPipe::SkeletalMesh);
#endif // RHI_RAYTRACING

	bResourcesInitialized = false;
}

TArray<FTransform>* FSkeletalMeshObjectCPUSkin::GetComponentSpaceTransforms() const
{
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	if(DynamicData)
	{
		return &(DynamicData->MeshComponentSpaceTransforms);
	}
	else
#endif
	{
		return NULL;
	}
}

TConstArrayView<FMatrix44f> FSkeletalMeshObjectCPUSkin::GetReferenceToLocalMatrices() const
{
	return DynamicData->ReferenceToLocal;
}

void FSkeletalMeshObjectCPUSkin::DrawVertexElements(FPrimitiveDrawInterface* PDI, const FMatrix& ToWorldSpace, bool bDrawNormals, bool bDrawTangents, bool bDrawBinormals) const
{
	uint32 NumIndices = CachedFinalVertices.Num();

	FMatrix LocalToWorldInverseTranspose = ToWorldSpace.InverseFast().GetTransposed();

	for (uint32 i = 0; i < NumIndices; i++)
	{
		FFinalSkinVertex& Vert = CachedFinalVertices[i];

		const FVector WorldPos = ToWorldSpace.TransformPosition( FVector(Vert.Position) );

		const FVector Normal = Vert.TangentZ.ToFVector();
		const FVector Tangent = Vert.TangentX.ToFVector();
		const FVector Binormal = FVector(Normal) ^ FVector(Tangent);

		const float Len = 1.0f;

		if( bDrawNormals )
		{
			PDI->DrawLine( WorldPos, WorldPos+LocalToWorldInverseTranspose.TransformVector( (FVector)(Normal) ).GetSafeNormal() * Len, FLinearColor( 0.0f, 1.0f, 0.0f), SDPG_World );
		}

		if( bDrawTangents )
		{
			PDI->DrawLine( WorldPos, WorldPos+LocalToWorldInverseTranspose.TransformVector( Tangent ).GetSafeNormal() * Len, FLinearColor( 1.0f, 0.0f, 0.0f), SDPG_World );
		}

		if( bDrawBinormals )
		{
			PDI->DrawLine( WorldPos, WorldPos+LocalToWorldInverseTranspose.TransformVector( Binormal ).GetSafeNormal() * Len, FLinearColor( 0.0f, 0.0f, 1.0f), SDPG_World );
		}
	}
}

/*-----------------------------------------------------------------------------
FDynamicSkelMeshObjectDataCPUSkin
-----------------------------------------------------------------------------*/

FDynamicSkelMeshObjectDataCPUSkin::FDynamicSkelMeshObjectDataCPUSkin(
	const FSkinnedMeshSceneProxyDynamicData& InDynamicData,
	const USkinnedAsset* InSkinnedAsset,
	FSkeletalMeshRenderData* InSkelMeshRenderData,
	int32 InLODIndex,
	const FMorphTargetWeightMap& InActiveMorphTargets,
	const TArrayView<const float>& InMorphTargetWeights)
:	LODIndex(InLODIndex)
,	ActiveMorphTargets(InActiveMorphTargets)
,	MorphTargetWeights(InMorphTargetWeights)
,	ClothBlendWeight(0.0f)
{
	if (InDynamicData.bUseReferenceToLocalMatrices)
	{
		ReferenceToLocal = InDynamicData.ReferenceToLocalMatrices;
	}
	else
	{
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
		MeshComponentSpaceTransforms = InDynamicData.GetComponentSpaceTransforms();

		const bool bCalculateComponentSpaceTransformsFromLeader = MeshComponentSpaceTransforms.IsEmpty(); // This will be empty for follower components.
		TArray<FTransform>* const LeaderBoneMappedMeshComponentSpaceTransforms = bCalculateComponentSpaceTransformsFromLeader ? &MeshComponentSpaceTransforms : nullptr;
#else
		TArray<FTransform>* const LeaderBoneMappedMeshComponentSpaceTransforms = nullptr;
#endif

		UpdateRefToLocalMatrices( ReferenceToLocal, InDynamicData, InSkinnedAsset, InSkelMeshRenderData, LODIndex, nullptr, LeaderBoneMappedMeshComponentSpaceTransforms );
	}

	// Update the clothing simulation mesh positions and normals
	FMatrix LocalToWorld;
	if (const IClothSimulationDataProvider* Provider = InDynamicData.GetClothSimulationDataProvider())
	{
		Provider->GetUpdateClothSimulationData_AnyThread(ClothSimulUpdateData, LocalToWorld, ClothBlendWeight);
	}
	else
	{
		ClothSimulUpdateData.Reset();
		LocalToWorld = FMatrix::Identity;
		ClothBlendWeight = 0.f;
	}

	WorldToLocal = LocalToWorld.InverseFast();
	if (!IsSkeletalMeshClothBlendEnabled())
	{
		ClothBlendWeight = 0.f;
	}
}

FDynamicSkelMeshObjectDataCPUSkin::~FDynamicSkelMeshObjectDataCPUSkin() = default;

/*-----------------------------------------------------------------------------
	FSkeletalMeshObjectCPUSkin - morph target blending implementation
-----------------------------------------------------------------------------*/

/** Struct used to hold temporary info during morph target blending */
struct FMorphTargetInfo
{
	/** The index into the morph weight list */
	int32						WeightIndex = INDEX_NONE;
	
	/** An iterator into the list of morph target deltas, depending on build, this can either be the raw
	 *  vertex deltas or the compressed ones, generated at cook time. */
	FMorphTargetDeltaIterator	VertexDeltaIterator;
};

/**
 *	Init set of info structs to hold temporary state while blending morph targets in.
 * @return							number of active morphs that are valid
 */
static uint32 InitEvalInfos(const FMorphTargetWeightMap& InActiveMorphTargets, const TArray<float>& MorphTargetWeights, int32 LODIndex, TArray<FMorphTargetInfo>& OutEvalInfos)
{
	uint32 NumValidMorphTargets=0;

	const float MorphTargetMaxBlendWeight = UE::SkeletalRender::Settings::GetMorphTargetMaxBlendWeight();

	for(const TTuple<const UMorphTarget*, int32>& MorphItem: InActiveMorphTargets)
	{
		FMorphTargetInfo NewInfo;
		const UMorphTarget* MorphTarget = MorphItem.Key;
		const int32 WeightIndex = MorphItem.Value;

		const float ActiveMorphAbsVertexWeight = FMath::Abs(MorphTargetWeights[WeightIndex]);

		if( MorphTarget != nullptr &&
			ActiveMorphAbsVertexWeight >= MinMorphTargetBlendWeight &&
			ActiveMorphAbsVertexWeight <= MorphTargetMaxBlendWeight &&
			MorphItem.Key->HasDataForLOD(LODIndex) )
		{
			// start at the first vertex since they affect base mesh verts in ascending order
			NewInfo.WeightIndex = WeightIndex;
			NewInfo.VertexDeltaIterator = MorphTarget->GetDeltaIteratorForLOD(LODIndex);

			NumValidMorphTargets++;
		}

		OutEvalInfos.Add(NewInfo);
	}
	return NumValidMorphTargets;
}

/** Release any state for the morphs being evaluated */
void TermEvalInfos(TArray<FMorphTargetInfo>& EvalInfos)
{
	EvalInfos.Empty();
}

/** 
* Derive the tanget/binormal using the new normal and the base tangent vectors for a vertex 
*/
template<typename VertexType>
FORCEINLINE void RebuildTangentBasis( VertexType& DestVertex )
{
	// derive the new tangent by orthonormalizing the new normal against
	// the base tangent vector (assuming these are normalized)
	FVector Tangent = DestVertex.TangentX.ToFVector();
	FVector Normal = DestVertex.TangentZ.ToFVector();
	Tangent = Tangent - ((Tangent | Normal) * Normal);
	Tangent.Normalize();
	DestVertex.TangentX = Tangent;
}

/**
* Applies the vertex deltas to a vertex.
*/
template<typename VertexType>
FORCEINLINE void ApplyMorphBlend( VertexType& DestVertex, const FMorphTargetDelta& SrcMorph, float Weight )
{
	// Add position offset 
	DestVertex.Position += SrcMorph.PositionDelta * Weight;

	// Save W before = operator. That overwrites W to be 127.
	int8 W = DestVertex.TangentZ.Vector.W;

	FVector TanZ = DestVertex.TangentZ.ToFVector();

	// add normal offset. can only apply normal deltas up to a weight of 1
	DestVertex.TangentZ = (TanZ + FVector(SrcMorph.TangentZDelta * FMath::Min(Weight,1.0f))).GetUnsafeNormal();
	// Recover W
	DestVertex.TangentZ.Vector.W = W;
} 

/**
* Blends the source vertex with all the active morph targets.
*/
template<typename VertexType>
FORCEINLINE void UpdateMorphedVertex( VertexType& MorphedVertex, const VertexType& SrcVertex, int32 CurBaseVertIdx, int32 LODIndex, TArray<FMorphTargetInfo>& EvalInfos, const TArray<float>& MorphWeights )
{
	MorphedVertex = SrcVertex;

	// iterate over all active morphs
	for( int32 MorphIdx=0; MorphIdx < EvalInfos.Num(); MorphIdx++ )
	{
		FMorphTargetInfo& Info = EvalInfos[MorphIdx];
		
		if (!Info.VertexDeltaIterator.AtEnd() && Info.VertexDeltaIterator->SourceIdx == CurBaseVertIdx)
		{
			ApplyMorphBlend(MorphedVertex, *Info.VertexDeltaIterator, MorphWeights[Info.WeightIndex]);
			++Info.VertexDeltaIterator;
		}
	}

	// rebuild orthonormal tangents
	RebuildTangentBasis( MorphedVertex );
}



/*-----------------------------------------------------------------------------
	FSkeletalMeshObjectCPUSkin - optimized skinning code
-----------------------------------------------------------------------------*/

MSVC_PRAGMA(warning(push))
MSVC_PRAGMA(warning(disable : 4730)) //mixing _m64 and floating point expressions may result in incorrect code

const VectorRegister		VECTOR_0001				= DECLARE_VECTOR_REGISTER(0.f, 0.f, 0.f, 1.f);

#define FIXED_VERTEX_INDEX 0xFFFF

template<typename VertexType, int32 NumberOfUVs>
static void SkinVertexSection(
	FFinalSkinVertex*& DestVertex,
	TArray<FMorphTargetInfo>& MorphEvalInfos,
	const TArray<float>& MorphWeights,
	const FSkelMeshRenderSection& Section,
	const FSkeletalMeshLODRenderData &LOD,
	FSkinWeightVertexBuffer& WeightBuffer,
	int32 VertexBufferBaseIndex, 
	uint32 NumValidMorphs, 
	int32 &CurBaseVertIdx, 
	int32 LODIndex, 
	const FMatrix44f* RESTRICT ReferenceToLocal, 
	const FClothSimulData* ClothSimData, 
	float ClothBlendWeight, 
	const FMatrix& WorldToLocal,
	const FVector& WorldScaleAbs )
{
	static constexpr VectorRegister VECTOR_INV_65535 = MakeVectorRegisterDoubleConstant(1.0 / 65535, 1.0 / 65535, 1.0 / 65535, 1.0 / 65535);
	
	// VertexCopy for morph. Need to allocate right struct
	// To avoid re-allocation, create 2 statics, and assign right struct
	VertexType  VertexCopy;

	// Prefetch all bone indices
	const FBoneIndexType* BoneMap = Section.BoneMap.GetData();
	FPlatformMisc::Prefetch( BoneMap );
	FPlatformMisc::Prefetch( BoneMap, PLATFORM_CACHE_LINE_SIZE );

	const int32 MaxSectionBoneInfluences = WeightBuffer.GetMaxBoneInfluences();
	const bool bLODUsesCloth = LOD.HasClothData() && ClothSimData != nullptr && ClothBlendWeight > 0.0f;
	const int32 NumSoftVertices = Section.GetNumVertices();
	if (NumSoftVertices > 0)
	{
		INC_DWORD_STAT_BY(STAT_CPUSkinVertices,NumSoftVertices);
		for(int32 VertexIndex = VertexBufferBaseIndex;VertexIndex < NumSoftVertices;VertexIndex++,DestVertex++)
		{
			const int32 VertexBufferIndex = Section.GetVertexBufferIndex() + VertexIndex;

			VertexType SrcSoftVertex;
			const FVector& VertexPosition = (FVector)LOD.StaticVertexBuffers.PositionVertexBuffer.VertexPosition(VertexBufferIndex);
			FPlatformMisc::Prefetch(&VertexPosition, PLATFORM_CACHE_LINE_SIZE);	// Prefetch next vertices
			
			SrcSoftVertex.Position = (FVector3f)VertexPosition;
			SrcSoftVertex.TangentX = LOD.StaticVertexBuffers.StaticMeshVertexBuffer.VertexTangentX(VertexBufferIndex);
			SrcSoftVertex.TangentZ = LOD.StaticVertexBuffers.StaticMeshVertexBuffer.VertexTangentZ(VertexBufferIndex);
			for (uint32 j = 0; j < VertexType::NumTexCoords; j++)
			{
				SrcSoftVertex.UVs[j] = LOD.StaticVertexBuffers.StaticMeshVertexBuffer.GetVertexUV_Typed<VertexType::StaticMeshVertexUVType>(VertexBufferIndex, j);
			}

			VertexType* MorphedVertex = &SrcSoftVertex;

			FSkinWeightInfo SrcWeights = WeightBuffer.GetVertexSkinWeights(VertexBufferIndex);

			if( NumValidMorphs ) 
			{
				MorphedVertex = &VertexCopy;
				UpdateMorphedVertex<VertexType>( *MorphedVertex, SrcSoftVertex, CurBaseVertIdx, LODIndex, MorphEvalInfos, MorphWeights);
			}

			const FMeshToMeshVertData* ClothVertData = nullptr;
			if (bLODUsesCloth)
			{
				constexpr int32 ClothLODBias = 0;  // Use base Cloth LOD mapping data (biased mappings are only required for GPU skinning of raytraced elements)
				ClothVertData = &Section.ClothMappingDataLODs[ClothLODBias][VertexIndex];
				FPlatformMisc::Prefetch(ClothVertData, PLATFORM_CACHE_LINE_SIZE);	// Prefetch next cloth vertex
			}

			const FBoneIndexType* RESTRICT BoneIndices = SrcWeights.InfluenceBones;
			const uint16* RESTRICT BoneWeights = SrcWeights.InfluenceWeights;

			static VectorRegister	SrcNormals[3];
			VectorRegister			DstNormals[3];
			SrcNormals[0] = VectorLoadFloat3_W1( &MorphedVertex->Position);
			SrcNormals[1] = Unpack3( &MorphedVertex->TangentX.Vector.Packed );
			SrcNormals[2] = Unpack4( &MorphedVertex->TangentZ.Vector.Packed );
			VectorRegister Weights = VectorMultiply( VectorLoadURGBA16N(BoneWeights), VECTOR_INV_65535 );
			VectorRegister ExtraWeights = MakeVectorRegister(0.f, 0.f, 0.f, 0.f);
			VectorRegister ExtraWeights2 = MakeVectorRegister(0.f, 0.f, 0.f, 0.f);
			if (MaxSectionBoneInfluences > 4)
			{
				ExtraWeights = VectorMultiply( VectorLoadURGBA16N(&BoneWeights[MAX_INFLUENCES_PER_STREAM]), VECTOR_INV_65535 );
			}
			if (MaxSectionBoneInfluences > 8)
			{
				ExtraWeights2 = VectorMultiply(VectorLoadURGBA16N(&BoneWeights[EXTRA_BONE_INFLUENCES]), VECTOR_INV_65535);
			}
			VectorResetFloatRegisters(); // Need to call this to be able to use regular floating point registers again after Unpack and VectorLoadByte4.

			const FMatrix44f BoneMatrix0 = ReferenceToLocal[BoneMap[BoneIndices[INFLUENCE_0]]];
			VectorRegister Weight0 = VectorReplicate( Weights, INFLUENCE_0 );
			VectorRegister M00	= VectorMultiply( VectorLoadAligned( &BoneMatrix0.M[0][0] ), Weight0 );
			VectorRegister M10	= VectorMultiply( VectorLoadAligned( &BoneMatrix0.M[1][0] ), Weight0 );
			VectorRegister M20	= VectorMultiply( VectorLoadAligned( &BoneMatrix0.M[2][0] ), Weight0 );
			VectorRegister M30	= VectorMultiply( VectorLoadAligned( &BoneMatrix0.M[3][0] ), Weight0 );

			if (MaxSectionBoneInfluences > 1 )
			{
				const FMatrix44f BoneMatrix1 = ReferenceToLocal[BoneMap[BoneIndices[INFLUENCE_1]]];
				VectorRegister Weight1 = VectorReplicate( Weights, INFLUENCE_1 );
				M00	= VectorMultiplyAdd( VectorLoadAligned( &BoneMatrix1.M[0][0] ), Weight1, M00 );
				M10	= VectorMultiplyAdd( VectorLoadAligned( &BoneMatrix1.M[1][0] ), Weight1, M10 );
				M20	= VectorMultiplyAdd( VectorLoadAligned( &BoneMatrix1.M[2][0] ), Weight1, M20 );
				M30	= VectorMultiplyAdd( VectorLoadAligned( &BoneMatrix1.M[3][0] ), Weight1, M30 );

				if (MaxSectionBoneInfluences > 2 )
				{
					const FMatrix44f BoneMatrix2 = ReferenceToLocal[BoneMap[BoneIndices[INFLUENCE_2]]];
					VectorRegister Weight2 = VectorReplicate( Weights, INFLUENCE_2 );
					M00	= VectorMultiplyAdd( VectorLoadAligned( &BoneMatrix2.M[0][0] ), Weight2, M00 );
					M10	= VectorMultiplyAdd( VectorLoadAligned( &BoneMatrix2.M[1][0] ), Weight2, M10 );
					M20	= VectorMultiplyAdd( VectorLoadAligned( &BoneMatrix2.M[2][0] ), Weight2, M20 );
					M30	= VectorMultiplyAdd( VectorLoadAligned( &BoneMatrix2.M[3][0] ), Weight2, M30 );

					if (MaxSectionBoneInfluences > 3 )
					{
						const FMatrix44f BoneMatrix3 = ReferenceToLocal[BoneMap[BoneIndices[INFLUENCE_3]]];
						VectorRegister Weight3 = VectorReplicate( Weights, INFLUENCE_3 );
						M00	= VectorMultiplyAdd( VectorLoadAligned( &BoneMatrix3.M[0][0] ), Weight3, M00 );
						M10	= VectorMultiplyAdd( VectorLoadAligned( &BoneMatrix3.M[1][0] ), Weight3, M10 );
						M20	= VectorMultiplyAdd( VectorLoadAligned( &BoneMatrix3.M[2][0] ), Weight3, M20 );
						M30	= VectorMultiplyAdd( VectorLoadAligned( &BoneMatrix3.M[3][0] ), Weight3, M30 );
					}

					if (MaxSectionBoneInfluences > 4)
					{
						const FMatrix44f BoneMatrix4 = ReferenceToLocal[BoneMap[BoneIndices[INFLUENCE_4]]];
						VectorRegister Weight4 = VectorReplicate( ExtraWeights, INFLUENCE_4 - INFLUENCE_4 );
						M00	= VectorMultiplyAdd( VectorLoadAligned( &BoneMatrix4.M[0][0] ), Weight4, M00 );
						M10	= VectorMultiplyAdd( VectorLoadAligned( &BoneMatrix4.M[1][0] ), Weight4, M10 );
						M20	= VectorMultiplyAdd( VectorLoadAligned( &BoneMatrix4.M[2][0] ), Weight4, M20 );
						M30	= VectorMultiplyAdd( VectorLoadAligned( &BoneMatrix4.M[3][0] ), Weight4, M30 );

						if (MaxSectionBoneInfluences > 5)
						{
							const FMatrix44f BoneMatrix5 = ReferenceToLocal[BoneMap[BoneIndices[INFLUENCE_5]]];
							VectorRegister Weight5 = VectorReplicate( ExtraWeights, INFLUENCE_5 - INFLUENCE_4 );
							M00	= VectorMultiplyAdd( VectorLoadAligned( &BoneMatrix5.M[0][0] ), Weight5, M00 );
							M10	= VectorMultiplyAdd( VectorLoadAligned( &BoneMatrix5.M[1][0] ), Weight5, M10 );
							M20	= VectorMultiplyAdd( VectorLoadAligned( &BoneMatrix5.M[2][0] ), Weight5, M20 );
							M30	= VectorMultiplyAdd( VectorLoadAligned( &BoneMatrix5.M[3][0] ), Weight5, M30 );

							if (MaxSectionBoneInfluences > 6)
							{
								const FMatrix44f BoneMatrix6 = ReferenceToLocal[BoneMap[BoneIndices[INFLUENCE_6]]];
								VectorRegister Weight6 = VectorReplicate( ExtraWeights, INFLUENCE_6 - INFLUENCE_4 );
								M00	= VectorMultiplyAdd( VectorLoadAligned( &BoneMatrix6.M[0][0] ), Weight6, M00 );
								M10	= VectorMultiplyAdd( VectorLoadAligned( &BoneMatrix6.M[1][0] ), Weight6, M10 );
								M20	= VectorMultiplyAdd( VectorLoadAligned( &BoneMatrix6.M[2][0] ), Weight6, M20 );
								M30	= VectorMultiplyAdd( VectorLoadAligned( &BoneMatrix6.M[3][0] ), Weight6, M30 );

								if (MaxSectionBoneInfluences > 7)
								{
									const FMatrix44f BoneMatrix7 = ReferenceToLocal[BoneMap[BoneIndices[INFLUENCE_7]]];
									VectorRegister Weight7 = VectorReplicate( ExtraWeights, INFLUENCE_7 - INFLUENCE_4 );
									M00	= VectorMultiplyAdd( VectorLoadAligned( &BoneMatrix7.M[0][0] ), Weight7, M00 );
									M10	= VectorMultiplyAdd( VectorLoadAligned( &BoneMatrix7.M[1][0] ), Weight7, M10 );
									M20	= VectorMultiplyAdd( VectorLoadAligned( &BoneMatrix7.M[2][0] ), Weight7, M20 );
									M30	= VectorMultiplyAdd( VectorLoadAligned( &BoneMatrix7.M[3][0] ), Weight7, M30 );

									if (MaxSectionBoneInfluences > 8)
									{
										const FMatrix44f BoneMatrix8 = ReferenceToLocal[BoneMap[BoneIndices[INFLUENCE_8]]];
										VectorRegister Weight8 = VectorReplicate( ExtraWeights2, INFLUENCE_8 - INFLUENCE_8 );
										M00 = VectorMultiplyAdd(VectorLoadAligned( &BoneMatrix8.M[0][0]), Weight8, M00 );
										M10 = VectorMultiplyAdd(VectorLoadAligned( &BoneMatrix8.M[1][0]), Weight8, M10 );
										M20 = VectorMultiplyAdd(VectorLoadAligned( &BoneMatrix8.M[2][0]), Weight8, M20 );
										M30 = VectorMultiplyAdd(VectorLoadAligned( &BoneMatrix8.M[3][0]), Weight8, M30 );

										if (MaxSectionBoneInfluences > 9)
										{
											const FMatrix44f BoneMatrix9 = ReferenceToLocal[BoneMap[BoneIndices[INFLUENCE_9]]];
											VectorRegister Weight9 = VectorReplicate(ExtraWeights2, INFLUENCE_9 - INFLUENCE_8);
											M00 = VectorMultiplyAdd(VectorLoadAligned(&BoneMatrix9.M[0][0]), Weight9, M00);
											M10 = VectorMultiplyAdd(VectorLoadAligned(&BoneMatrix9.M[1][0]), Weight9, M10);
											M20 = VectorMultiplyAdd(VectorLoadAligned(&BoneMatrix9.M[2][0]), Weight9, M20);
											M30 = VectorMultiplyAdd(VectorLoadAligned(&BoneMatrix9.M[3][0]), Weight9, M30);

											if (MaxSectionBoneInfluences > 10)
											{
												const FMatrix44f BoneMatrix10 = ReferenceToLocal[BoneMap[BoneIndices[INFLUENCE_10]]];
												VectorRegister Weight10 = VectorReplicate(ExtraWeights2, INFLUENCE_10 - INFLUENCE_8);
												M00 = VectorMultiplyAdd(VectorLoadAligned(&BoneMatrix10.M[0][0]), Weight10, M00);
												M10 = VectorMultiplyAdd(VectorLoadAligned(&BoneMatrix10.M[1][0]), Weight10, M10);
												M20 = VectorMultiplyAdd(VectorLoadAligned(&BoneMatrix10.M[2][0]), Weight10, M20);
												M30 = VectorMultiplyAdd(VectorLoadAligned(&BoneMatrix10.M[3][0]), Weight10, M30);

												if (MaxSectionBoneInfluences > 11)
												{
													const FMatrix44f BoneMatrix11 = ReferenceToLocal[BoneMap[BoneIndices[INFLUENCE_11]]];
													VectorRegister Weight11 = VectorReplicate(ExtraWeights2, INFLUENCE_11 - INFLUENCE_8);
													M00 = VectorMultiplyAdd(VectorLoadAligned(&BoneMatrix11.M[0][0]), Weight11, M00);
													M10 = VectorMultiplyAdd(VectorLoadAligned(&BoneMatrix11.M[1][0]), Weight11, M10);
													M20 = VectorMultiplyAdd(VectorLoadAligned(&BoneMatrix11.M[2][0]), Weight11, M20);
													M30 = VectorMultiplyAdd(VectorLoadAligned(&BoneMatrix11.M[3][0]), Weight11, M30);
												}
											}
										}
									}
								}
							}
						}
					}
				}
			}

			VectorRegister N_xxxx = VectorReplicate( SrcNormals[0], 0 );
			VectorRegister N_yyyy = VectorReplicate( SrcNormals[0], 1 );
			VectorRegister N_zzzz = VectorReplicate( SrcNormals[0], 2 );
			DstNormals[0] = VectorMultiplyAdd( N_xxxx, M00, VectorMultiplyAdd( N_yyyy, M10, VectorMultiplyAdd( N_zzzz, M20, M30 ) ) );

			DstNormals[1] = VectorZero();
			N_xxxx = VectorReplicate( SrcNormals[1], 0 );
			N_yyyy = VectorReplicate( SrcNormals[1], 1 );
			N_zzzz = VectorReplicate( SrcNormals[1], 2 );
			DstNormals[1] = VectorNormalize(VectorMultiplyAdd( N_xxxx, M00, VectorMultiplyAdd( N_yyyy, M10, VectorMultiply( N_zzzz, M20 ) ) ));

			N_xxxx = VectorReplicate( SrcNormals[2], 0 );
			N_yyyy = VectorReplicate( SrcNormals[2], 1 );
			N_zzzz = VectorReplicate( SrcNormals[2], 2 );
			DstNormals[2] = VectorZero();
			DstNormals[2] = VectorNormalize(VectorMultiplyAdd( N_xxxx, M00, VectorMultiplyAdd( N_yyyy, M10, VectorMultiply( N_zzzz, M20 ) ) ));


			// carry over the W component (sign of basis determinant) 
			DstNormals[2] = VectorMultiplyAdd( VECTOR_0001, SrcNormals[2], DstNormals[2] );

			// Write to memory:
			VectorStoreFloat3( DstNormals[0], &DestVertex->Position );
			Pack3( DstNormals[1], &DestVertex->TangentX.Vector.Packed );
			Pack4( DstNormals[2], &DestVertex->TangentZ.Vector.Packed );
			VectorResetFloatRegisters(); // Need to call this to be able to use regular floating point registers again after Pack().

			// Apply cloth. This code has been adapted from GpuSkinVertexFactory.usf
			if (ClothVertData != nullptr && ClothVertData->SourceMeshVertIndices[3] < FIXED_VERTEX_INDEX)
			{
				struct ClothCPU
				{
					FORCEINLINE static FVector GetClothSimulPosition(const FClothSimulData& InClothSimData, int32 InIndex)
					{
						if (InClothSimData.Positions.IsValidIndex(InIndex))
						{
							return FVector(InClothSimData.Transform.TransformPosition((FVector)InClothSimData.Positions[InIndex]));
						}

						return FVector::ZeroVector;
					}

					FORCEINLINE static FVector GetClothSimulNormal(const FClothSimulData& InClothSimData, int32 InIndex)
					{
						if (InClothSimData.Normals.IsValidIndex(InIndex))
						{
							return FVector(InClothSimData.Transform.TransformVector((FVector)InClothSimData.Normals[InIndex]));
						}

						return FVector(0, 0, 1);
					}

					FORCEINLINE static FVector ClothingPosition(const FMeshToMeshVertData& InClothVertData, const FClothSimulData& InClothSimData, const FVector& InWorldScaleAbs)
					{
						return    InClothVertData.PositionBaryCoordsAndDist.X * (GetClothSimulPosition(InClothSimData, InClothVertData.SourceMeshVertIndices[0]) + GetClothSimulNormal(InClothSimData, InClothVertData.SourceMeshVertIndices[0]) * InClothVertData.PositionBaryCoordsAndDist.W * InWorldScaleAbs.X)
								+ InClothVertData.PositionBaryCoordsAndDist.Y * (GetClothSimulPosition(InClothSimData, InClothVertData.SourceMeshVertIndices[1]) + GetClothSimulNormal(InClothSimData, InClothVertData.SourceMeshVertIndices[1]) * InClothVertData.PositionBaryCoordsAndDist.W * InWorldScaleAbs.Y)
								+ InClothVertData.PositionBaryCoordsAndDist.Z * (GetClothSimulPosition(InClothSimData, InClothVertData.SourceMeshVertIndices[2]) + GetClothSimulNormal(InClothSimData, InClothVertData.SourceMeshVertIndices[2]) * InClothVertData.PositionBaryCoordsAndDist.W * InWorldScaleAbs.Z);
					}

					FORCEINLINE static void ClothingTangents(const FMeshToMeshVertData& InClothVertData, const FClothSimulData& InClothSimData, const FVector& InSimulatedPosition, const FMatrix& InWorldToLocal, const FVector& InWorldScaleAbs, FVector& OutTangentX, FVector& OutTangentZ)
					{
						FVector A = GetClothSimulPosition(InClothSimData, InClothVertData.SourceMeshVertIndices[0]);
						FVector B = GetClothSimulPosition(InClothSimData, InClothVertData.SourceMeshVertIndices[1]);
						FVector C = GetClothSimulPosition(InClothSimData, InClothVertData.SourceMeshVertIndices[2]);

						FVector NA = GetClothSimulNormal(InClothSimData, InClothVertData.SourceMeshVertIndices[0]);
						FVector NB = GetClothSimulNormal(InClothSimData, InClothVertData.SourceMeshVertIndices[1]);
						FVector NC = GetClothSimulNormal(InClothSimData, InClothVertData.SourceMeshVertIndices[2]);

						FVector NormalPosition = InClothVertData.NormalBaryCoordsAndDist.X*(A + NA*InClothVertData.NormalBaryCoordsAndDist.W * InWorldScaleAbs.X)
												+ InClothVertData.NormalBaryCoordsAndDist.Y*(B + NB*InClothVertData.NormalBaryCoordsAndDist.W * InWorldScaleAbs.Y)
												+ InClothVertData.NormalBaryCoordsAndDist.Z*(C + NC*InClothVertData.NormalBaryCoordsAndDist.W * InWorldScaleAbs.Z);

						FVector TangentPosition = InClothVertData.TangentBaryCoordsAndDist.X*(A + NA*InClothVertData.TangentBaryCoordsAndDist.W * InWorldScaleAbs.X)
												+ InClothVertData.TangentBaryCoordsAndDist.Y*(B + NB*InClothVertData.TangentBaryCoordsAndDist.W * InWorldScaleAbs.Y)
												+ InClothVertData.TangentBaryCoordsAndDist.Z*(C + NC*InClothVertData.TangentBaryCoordsAndDist.W * InWorldScaleAbs.Z);

						OutTangentX = (TangentPosition - InSimulatedPosition).GetUnsafeNormal();
						OutTangentZ = (NormalPosition - InSimulatedPosition).GetUnsafeNormal();

						// cloth data are all in world space so need to change into local space
						OutTangentX = InWorldToLocal.TransformVector(OutTangentX);
						OutTangentZ = InWorldToLocal.TransformVector(OutTangentZ);
					}
				};

				// build sim position (in world space)
				FVector SimulatedPositionWorld = ClothCPU::ClothingPosition(*ClothVertData, *ClothSimData, WorldScaleAbs);

				// transform back to local space
				FVector3f SimulatedPosition = (FVector4f)WorldToLocal.TransformPosition(SimulatedPositionWorld);

				const float VertexBlend = ClothBlendWeight * (1.0f - (ClothVertData->SourceMeshVertIndices[3] / 65535.0f));
				
				// Lerp between skinned and simulated position
				DestVertex->Position = FMath::Lerp(DestVertex->Position, SimulatedPosition, VertexBlend);

				// recompute tangent & normal
				FVector TangentX;
				FVector TangentZ;
				ClothCPU::ClothingTangents(*ClothVertData, *ClothSimData, SimulatedPositionWorld, WorldToLocal, WorldScaleAbs, TangentX, TangentZ);

				// Lerp between skinned and simulated tangents
				FVector SkinnedTangentX = DestVertex->TangentX.ToFVector();
				FVector4 SkinnedTangentZ = DestVertex->TangentZ.ToFVector4();
				DestVertex->TangentX = (TangentX * VertexBlend) + (SkinnedTangentX * (1.0f - VertexBlend));
				DestVertex->TangentZ = FVector4((TangentZ * VertexBlend) + (SkinnedTangentZ * (1.0f - VertexBlend)), SkinnedTangentZ.W);
			}

			// Copy UVs.
			for (int32 UVIndex = 0; UVIndex < NumberOfUVs; ++UVIndex)
			{
				DestVertex->TextureCoordinates[UVIndex] = FVector2D(LOD.StaticVertexBuffers.StaticMeshVertexBuffer.GetVertexUV(Section.GetVertexBufferIndex() + VertexIndex, UVIndex));
			}

			CurBaseVertIdx++;
		}
	}
}

template<typename VertexType, int32 NumberOfUVs>
static void SkinVertices(
	FFinalSkinVertex* DestVertex, 
	FMatrix44f* ReferenceToLocal, 
	int32 LODIndex, 
	FSkeletalMeshLODRenderData& LOD,
	FSkinWeightVertexBuffer& WeightBuffer,
	const FMorphTargetWeightMap& InActiveMorphTargets, 
	const TArray<float>& MorphTargetWeights, 
	const TMap<int32, FClothSimulData>& ClothSimulUpdateData, 
	float ClothBlendWeight, 
	const FMatrix& WorldToLocal,
	const FVector& WorldScale)
{
	uint32 StatusRegister = VectorGetControlRegister();
	VectorSetControlRegister( StatusRegister | VECTOR_ROUND_TOWARD_ZERO );

	// Create array to track state during morph blending
	TArray<FMorphTargetInfo> MorphEvalInfos;
	uint32 NumValidMorphs = InitEvalInfos(InActiveMorphTargets, MorphTargetWeights, LODIndex, MorphEvalInfos);

	for ( uint32 MatrixIndex=0; MatrixIndex < (uint32)LOD.RequiredBones.Num(); MatrixIndex+=2 )
	{
		FPlatformMisc::Prefetch( ReferenceToLocal + MatrixIndex );
	}

	int32 CurBaseVertIdx = 0;

	int32 VertexBufferBaseIndex=0;

	const FVector WorldScaleAbs = WorldScale.GetAbs();  // World scale can't be used mirrored to calculate the clothing positions and tangents since the cloth normals are then reversed

	for(int32 SectionIndex= 0;SectionIndex< LOD.RenderSections.Num();SectionIndex++)
	{
		FSkelMeshRenderSection& Section = LOD.RenderSections[SectionIndex];

		const FClothSimulData* ClothSimData = ClothSimulUpdateData.Find(Section.CorrespondClothAssetIndex);

		SkinVertexSection<VertexType, NumberOfUVs>(DestVertex, MorphEvalInfos, MorphTargetWeights, Section, LOD, WeightBuffer, VertexBufferBaseIndex, NumValidMorphs, CurBaseVertIdx, LODIndex, ReferenceToLocal, ClothSimData, ClothBlendWeight, WorldToLocal, WorldScaleAbs);
	}

	VectorSetControlRegister( StatusRegister );
}

/**
 * Convert FPackedNormal to 0-1 FVector4
 */
FVector4 GetTangetToColor(FPackedNormal Tangent)
{
	VectorRegister VectorToUnpack = Tangent.GetVectorRegister();
	// Write to FVector and return it.
	FVector4 UnpackedVector;
	VectorStoreAligned( VectorToUnpack, &UnpackedVector );

	FVector4 Src = UnpackedVector;
	Src = Src + FVector4(1.f, 1.f, 1.f, 1.f);
	Src = Src/2.f;
	return Src;
}

/** 
 * Modify the vertex buffer to store bone weights in the UV coordinates for rendering 
 * @param DestVertex - already filled out vertex buffer from SkinVertices
 * @param LOD - LOD model corresponding to DestVertex 
 * @param BonesOfInterest - array of bones we want to display
 */
static FORCEINLINE void CalculateSectionBoneWeights(FFinalSkinVertex*& DestVertex, FSkinWeightVertexBuffer& SkinWeightVertexBuffer, FSkelMeshRenderSection& Section, const TArray<int32>& BonesOfInterest)
{
	int32 VertexBufferBaseIndex = 0;

	//array of bone mapping
	FBoneIndexType* BoneMap = Section.BoneMap.GetData();

	for(int32 VertexIndex = VertexBufferBaseIndex;VertexIndex < Section.GetNumVertices();VertexIndex++,DestVertex++)
	{
		const int32 VertexBufferIndex = Section.GetVertexBufferIndex() + VertexIndex;
		FSkinWeightInfo SrcWeight = SkinWeightVertexBuffer.GetVertexSkinWeights(VertexBufferIndex);

		//Zero out the UV coords
		DestVertex->TextureCoordinates[0].X = 0.0f;
		DestVertex->TextureCoordinates[0].Y = 0.0f;

		const FBoneIndexType* RESTRICT BoneIndices = SrcWeight.InfluenceBones;
		const uint16* RESTRICT BoneWeights = SrcWeight.InfluenceWeights;

		for (uint32 i = 0; i < SkinWeightVertexBuffer.GetMaxBoneInfluences(); i++)
		{
			if (BonesOfInterest.Contains(BoneMap[BoneIndices[i]]))
			{
				DestVertex->TextureCoordinates[0].X += BoneWeights[i] / 65535.0; 
				DestVertex->TextureCoordinates[0].Y += BoneWeights[i] / 65535.0;
			}
		}
	}
}

/** 
 * Modify the vertex buffer to store bone weights in the UV coordinates for rendering 
 * @param DestVertex - already filled out vertex buffer from SkinVertices
 * @param LOD - LOD render data corresponding to DestVertex 
 * @param WeightBuffer - skin weights
 * @param BonesOfInterest - array of bones we want to display
 */
static void CalculateBoneWeights(FFinalSkinVertex* DestVertex, FSkeletalMeshLODRenderData& LOD, FSkinWeightVertexBuffer& WeightBuffer, TArray<int32> InBonesOfInterest)
{
	const float INV255 = 1.f/255.f;

	int32 VertexBufferBaseIndex = 0;

	for(int32 SectionIndex= 0;SectionIndex< LOD.RenderSections.Num();SectionIndex++)
	{
		FSkelMeshRenderSection& Section = LOD.RenderSections[SectionIndex];
		CalculateSectionBoneWeights(DestVertex, WeightBuffer, Section, InBonesOfInterest);
	}
}

static void CalculateMorphTargetWeights(FFinalSkinVertex* DestVertex, FSkeletalMeshLODRenderData& LOD, int LODIndex, TArray<UMorphTarget*> InMorphTargetOfInterest)
{
	const FFinalSkinVertex* EndVert = DestVertex + LOD.GetNumVertices();

	for (FFinalSkinVertex* ClearVert = DestVertex; ClearVert != EndVert; ++ClearVert)
	{
		ClearVert->TextureCoordinates[0].X = 0.0f;
		ClearVert->TextureCoordinates[0].Y = 0.0f;
	}

	for (const UMorphTarget* MorphTarget : InMorphTargetOfInterest)
	{
		for (FMorphTargetDeltaIterator DeltaIt = MorphTarget->GetDeltaIteratorForLOD(LODIndex); !DeltaIt.AtEnd(); ++DeltaIt)
		{
			FFinalSkinVertex* SetVert = DestVertex + DeltaIt->SourceIdx;
			SetVert->TextureCoordinates[0].X += 1.0f;
			SetVert->TextureCoordinates[0].Y += 1.0f;
		}
	}
}


MSVC_PRAGMA(warning(pop))