// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	SkeletalRenderCPUSkin.h: CPU skinned mesh object and resource definitions
=============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "ProfilingDebugging/ResourceSize.h"
#include "RenderResource.h"
#include "RayTracingGeometry.h"
#include "LocalVertexFactory.h"
#include "Components/SkinnedMeshComponent.h"
#include "SkeletalRenderPublic.h"
#include "ClothingSystemRuntimeTypes.h"
#include "Rendering/SkeletalMeshRenderData.h"
#include "Rendering/SkeletalMeshLODRenderData.h"
#include "StaticMeshResources.h"

class FPrimitiveDrawInterface;
class UMorphTarget;

/** 
* Stores the updated matrices needed to skin the verts.
* Created by the game thread and sent to the rendering thread as an update 
*/
class FDynamicSkelMeshObjectDataCPUSkin
{
public:

	/**
	* Constructor
	* Updates the ReferenceToLocal matrices using the new dynamic data.
	* @param	InSkelMeshComponent - parent skel mesh component
	* @param	InLODIndex - each lod has its own bone map 
	* @param	InActiveMorphTargets - Active Morph Targets to blend with during skinning
	* @param	InMorphTargetWeights - All Morph Target weights to blend with during skinning
	*/
	ENGINE_API FDynamicSkelMeshObjectDataCPUSkin(
		const FSkinnedMeshSceneProxyDynamicData& InDynamicData,
		const USkinnedAsset* InSkinnedAsset,
		FSkeletalMeshRenderData* InSkelMeshRenderData,
		int32 InLODIndex,
		const FMorphTargetWeightMap& InActiveMorphTargets,
		const TArrayView<const float>& InMorphTargetWeights);

	ENGINE_API virtual ~FDynamicSkelMeshObjectDataCPUSkin();

	/** Local to world transform, used for cloth as sim data is in world space */
	FMatrix WorldToLocal;

	/** ref pose to local space transforms */
	TArray<FMatrix44f> ReferenceToLocal;

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST) 
	/** component space bone transforms*/
	TArray<FTransform> MeshComponentSpaceTransforms;
#endif
	/** currently LOD for bones being updated */
	int32 LODIndex;
	/** Morphs to blend when skinning verts */
	FMorphTargetWeightMap ActiveMorphTargets;
	/** Morph Weights to blend when skinning verts */
	TArray<float> MorphTargetWeights;

	/** data for updating cloth section */
	TMap<int32, FClothSimulData> ClothSimulUpdateData;

	/** a weight factor to blend between simulated positions and skinned positions */
	float ClothBlendWeight;

	/** Returns the size of memory allocated by render data */
	virtual void GetResourceSizeEx(FResourceSizeEx& CumulativeResourceSize)
 	{
		CumulativeResourceSize.AddDedicatedSystemMemoryBytes(sizeof(*this));
 		
		CumulativeResourceSize.AddDedicatedSystemMemoryBytes(ReferenceToLocal.GetAllocatedSize());
		CumulativeResourceSize.AddDedicatedSystemMemoryBytes(ActiveMorphTargets.GetAllocatedSize());
 	}
};

/**
 * Render data for a CPU skinned mesh
 */
class FSkeletalMeshObjectCPUSkin : public FSkeletalMeshObject
{
public:

	/** @param	InSkeletalMeshComponent - skeletal mesh primitive we want to render */
	ENGINE_API FSkeletalMeshObjectCPUSkin(const USkinnedMeshComponent* InMeshComponent, FSkeletalMeshRenderData* InSkelMeshRenderData, ERHIFeatureLevel::Type InFeatureLevel);
	ENGINE_API FSkeletalMeshObjectCPUSkin(const FSkinnedMeshSceneProxyDesc& InMeshDesc, FSkeletalMeshRenderData* InSkelMeshRenderData, ERHIFeatureLevel::Type InFeatureLevel);
	ENGINE_API virtual ~FSkeletalMeshObjectCPUSkin();

	//~ Begin FSkeletalMeshObject Interface
	ENGINE_API virtual void InitResources(const FSkinnedMeshSceneProxyDesc& InMeshDesc) override;
	ENGINE_API virtual void ReleaseResources() override;
	ENGINE_API virtual void Update(int32 LODIndex, const FSkinnedMeshSceneProxyDynamicData& InDynamicData, const FPrimitiveSceneProxy* InSceneProxy, const USkinnedAsset* InSkinnedAsset, const FMorphTargetWeightMap& InActiveMorphTargets, const TArray<float>& InMorphTargetWeights, EPreviousBoneTransformUpdateMode PreviousBoneTransformUpdateMode, const FExternalMorphWeightData& InExternalMorphWeightData) override;
	ENGINE_API void UpdateDynamicData_RenderThread(FRHICommandList& RHICmdList, FDynamicSkelMeshObjectDataCPUSkin* InDynamicData, uint64 FrameNumberToPrepare, uint32 RevisionNumber);
	ENGINE_API virtual void EnableOverlayRendering(bool bEnabled, const TArray<int32>* InBonesOfInterest, const TArray<UMorphTarget*>* InMorphTargetOfInterest) override;
	ENGINE_API virtual const FVertexFactory* GetSkinVertexFactory(const FSceneView* View, int32 LODIndex, int32 ChunkIdx, ESkinVertexFactoryMode VFMode = ESkinVertexFactoryMode::Default) const override;
	ENGINE_API virtual const FVertexFactory* GetStaticSkinVertexFactory(int32 LODIndex, int32 ChunkIdx, ESkinVertexFactoryMode VFMode) const override;
	ENGINE_API virtual TArray<FTransform>* GetComponentSpaceTransforms() const override;
	ENGINE_API virtual TConstArrayView<FMatrix44f> GetReferenceToLocalMatrices() const override;

	/**
	 * Re-skin cached vertices for an LOD and update the vertex buffer. Note that this
	 * function is called from the render thread!
	 * @param	LODIndex - index to LODs
	 * @param	bForce - force update even if LOD index hasn't changed
	 */
	ENGINE_API void CacheVertices(int32 LODIndex, bool bForce, FRHICommandList& RHICmdList) const;

	virtual int32 GetLOD() const override
	{
		if(DynamicData)
		{
			return DynamicData->LODIndex;
		}
		else
		{
			return 0;
		}
	}

	virtual bool HaveValidDynamicData() const override
	{ 
		return (DynamicData != nullptr); 
	}

	virtual void GetResourceSizeEx(FResourceSizeEx& CumulativeResourceSize) override
	{
		CumulativeResourceSize.AddDedicatedSystemMemoryBytes(sizeof(*this));

		if(DynamicData)
		{
			DynamicData->GetResourceSizeEx(CumulativeResourceSize);
		}

		CumulativeResourceSize.AddDedicatedSystemMemoryBytes(LODs.GetAllocatedSize()); 

		// include extra data from LOD
		for (int32 I=0; I<LODs.Num(); ++I)
		{
			LODs[I].GetResourceSizeEx(CumulativeResourceSize);
		}

		CumulativeResourceSize.AddDedicatedSystemMemoryBytes(CachedFinalVertices.GetAllocatedSize());
		CumulativeResourceSize.AddDedicatedSystemMemoryBytes(BonesOfInterest.GetAllocatedSize());
	}

	virtual bool IsCPUSkinned() const override { return true; }

	ENGINE_API virtual void DrawVertexElements(FPrimitiveDrawInterface* PDI, const FMatrix& ToWorldSpace, bool bDrawNormals, bool bDrawTangents, bool bDrawBinormals) const override;
	//~ End FSkeletalMeshObject Interface

	/** Access cached final vertices */
	const TArray<FFinalSkinVertex>& GetCachedFinalVertices() const { return CachedFinalVertices; }


	ENGINE_API virtual void UpdateSkinWeightBuffer(const TArrayView<const FSkelMeshComponentLODInfo> InLODInfo) override;

private:
	/** vertex data for rendering a single LOD */
	struct FSkeletalMeshObjectLOD
	{
		FSkeletalMeshRenderData* SkelMeshRenderData;
		// index into FSkeletalMeshRenderData::LODRenderData[]
		int32 LODIndex;

		mutable FLocalVertexFactory	VertexFactory;

		/** The buffer containing vertex data. */
		mutable FStaticMeshVertexBuffer StaticMeshVertexBuffer;
		/** The buffer containing the position vertex data. */
		mutable FPositionVertexBuffer PositionVertexBuffer;

		/** Skin weight buffer to use, could be from asset or component override */
		FSkinWeightVertexBuffer* MeshObjectWeightBuffer;

		/** Color buffer to use, could be from asset or component override */
		FColorVertexBuffer* MeshObjectColorBuffer;

#if RHI_RAYTRACING
		/** Geometry for ray tracing. */
		FRayTracingGeometry RayTracingGeometry;
#endif // RHI_RAYTRACING

		/** true if resources for this LOD have already been initialized. */
		bool						bResourcesInitialized;

		FSkeletalMeshObjectLOD(ERHIFeatureLevel::Type InFeatureLevel, FSkeletalMeshRenderData* InSkelMeshRenderData, int32 InLOD)
		:	SkelMeshRenderData(InSkelMeshRenderData)
		,	LODIndex(InLOD)
		,	VertexFactory(InFeatureLevel, "FSkeletalMeshObjectLOD")
		,	MeshObjectWeightBuffer(nullptr)
		,	MeshObjectColorBuffer(nullptr)
		,	bResourcesInitialized( false )
		{
		}
		/** 
		 * Init rendering resources for this LOD 
		 */
		void InitResources(const FSkelMeshComponentLODInfo* CompLODInfo);
		/** 
		 * Release rendering resources for this LOD 
		 */
		void ReleaseResources();

		/**
	 	 * Get Resource Size : return the size of Resource this allocates
	 	 */
		void GetResourceSizeEx(FResourceSizeEx& CumulativeResourceSize)
		{
			CumulativeResourceSize.AddUnknownMemoryBytes(StaticMeshVertexBuffer.GetResourceSize() + PositionVertexBuffer.GetStride() * PositionVertexBuffer.GetNumVertices());
		}

		void UpdateSkinWeights(const FSkelMeshComponentLODInfo* CompLODInfo);
	};

	/** Render data for each LOD */
	TArray<FSkeletalMeshObjectLOD> LODs;

protected:
	/** Data that is updated dynamically and is needed for rendering */
	class FDynamicSkelMeshObjectDataCPUSkin* DynamicData;

private:
 	/** Index of LOD level's vertices that are currently stored in CachedFinalVertices */
 	mutable int32	CachedVertexLOD;

 	/** Cached skinned vertices. Only updated/accessed by the rendering thread and exporters */
 	mutable TArray<FFinalSkinVertex> CachedFinalVertices;

	/** Array of bone's to render bone weights for */
	TArray<int32> BonesOfInterest;
	TArray<UMorphTarget*> MorphTargetOfInterest;

	/** Bone weight viewing in editor */
	bool bRenderOverlayMaterial;
};
