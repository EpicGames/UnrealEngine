// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	SkeletalRenderGPUSkin.cpp: GPU skinned skeletal mesh rendering code.
=============================================================================*/

#include "SkeletalRenderGPUSkin.h"
#include "Animation/MeshDeformerInstance.h"
#include "Components/SkeletalMeshComponent.h"
#include "Components/InstancedSkinnedMeshComponent.h"
#include "RenderUtils.h"
#include "SkeletalRender.h"
#include "GPUSkinCache.h"
#include "Rendering/RenderCommandPipes.h"
#include "ShaderParameterUtils.h"
#include "SceneInterface.h"
#include "SkeletalMeshSceneProxy.h"
#include "RenderGraphUtils.h"
#include "RenderCore.h"
#include "SkinnedMeshSceneProxyDesc.h"
#include "Engine/SkinnedAssetCommon.h"
#include "RHIResourceUtils.h"
#include "SkinningDefinitions.h"
#include "ComponentRecreateRenderStateContext.h"
#include <algorithm>

DEFINE_LOG_CATEGORY_STATIC(LogSkeletalGPUSkinMesh, Warning, All);

// 0/1
#define UPDATE_PER_BONE_DATA_ONLY_FOR_OBJECT_BEEN_VISIBLE 1

DECLARE_CYCLE_STAT(TEXT("Morph Vertex Buffer Update"),STAT_MorphVertexBuffer_Update,STATGROUP_MorphTarget);
DECLARE_CYCLE_STAT(TEXT("Morph Vertex Buffer Init"),STAT_MorphVertexBuffer_Init,STATGROUP_MorphTarget);
DECLARE_CYCLE_STAT(TEXT("Morph Vertex Buffer Apply Delta"),STAT_MorphVertexBuffer_ApplyDelta,STATGROUP_MorphTarget);
DECLARE_CYCLE_STAT(TEXT("Morph Vertex Buffer Alloc"), STAT_MorphVertexBuffer_Alloc, STATGROUP_MorphTarget);
DECLARE_CYCLE_STAT(TEXT("Morph Vertex Buffer RHI Lock and copy"), STAT_MorphVertexBuffer_RhiLockAndCopy, STATGROUP_MorphTarget);
DECLARE_CYCLE_STAT(TEXT("Morph Vertex Buffer RHI Unlock"), STAT_MorphVertexBuffer_RhiUnlock, STATGROUP_MorphTarget);
DECLARE_GPU_STAT_NAMED(MorphTargets, TEXT("Morph Target Compute"));

static TAutoConsoleVariable<int32> CVarMotionBlurDebug(
	TEXT("r.MotionBlurDebug"),
	0,
	TEXT("Defines if we log debugging output for motion blur rendering.\n")
	TEXT(" 0: off (default)\n")
	TEXT(" 1: on"),
	ECVF_Cheat | ECVF_RenderThreadSafe);

static bool GEnableMorphTargets = true;
static FAutoConsoleVariableRef CVarMorphTargetEnable(
	TEXT("r.MorphTarget.Enable"),
	GEnableMorphTargets,
	TEXT("Enables morph target rendering.\n")
	TEXT(" 0: Disable\n")
	TEXT(" 1: Enable (default)\n"),
	ECVF_RenderThreadSafe
	);

static bool GEnableCloth = true;
static FAutoConsoleVariableRef CVarClothEnable(
	TEXT("r.Cloth.Enable"),
	GEnableCloth,
	TEXT("Enables cloth rendering.\n")
	TEXT(" 0: Disable\n")
	TEXT(" 1: Enable (default)\n"),
	ECVF_RenderThreadSafe
	);

static int32 GUseGPUMorphTargets = 1;
static FAutoConsoleVariableRef CVarUseGPUMorphTargets(
	TEXT("r.MorphTarget.Mode"),
	GUseGPUMorphTargets,
	TEXT("Use GPU for computing morph targets.\n")
	TEXT(" 0: Use original CPU method (loop per morph then by vertex)\n")
	TEXT(" 1: Enable GPU method (default)\n"),
	ECVF_ReadOnly
	);

static int32 GForceUpdateMorphTargets = 0;
static FAutoConsoleVariableRef CVarForceUpdateMorphTargets(
	TEXT("r.MorphTarget.ForceUpdate"),
	GForceUpdateMorphTargets,
	TEXT("Force morph target deltas to be calculated every frame.\n")
	TEXT(" 0: Default\n")
	TEXT(" 1: Force Update\n"),
	ECVF_Default
	);

static int32 GGPUSkinBoneTransformAllocationMode = 1;
static FAutoConsoleVariableRef CVarGPUSkinBoneTransformAllocationMode(
	TEXT("r.GPUSkin.BoneTransformAllocationMode"),
	GGPUSkinBoneTransformAllocationMode,
	TEXT("Where to prefer to allocate bone transforms for non-Nanite skeletal meshes.")
	TEXT(" 0 - Vertex Factory: Per-section bone buffers are allocated by the Vertex Factory all the time")
	TEXT(" 1 - Scene:          Bone transforms are allocated in the scene extension when possible."),
	FConsoleVariableDelegate::CreateLambda([](IConsoleVariable* InVariable)
	{
		FGlobalComponentRecreateRenderStateContext Context;
	}),
	ECVF_RenderThreadSafe
);

EBoneTransformAllocationMode GetBoneTransformAllocationMode()
{
	return GGPUSkinBoneTransformAllocationMode == 0 || !IsGPUSkinSceneExtensionEnabled()
		? EBoneTransformAllocationMode::VertexFactory
		: EBoneTransformAllocationMode::Scene;
}

bool GGPUSkinSceneExtensionEnabled = true;
static FAutoConsoleVariableRef CVarGPUSkinUseSceneExtension(
	TEXT("r.GPUSkin.UseSceneExtension"),
	GGPUSkinSceneExtensionEnabled,
	TEXT("When set, allows GPU skin skeletal meshes to be registered with the skinning scene extension."),
	ECVF_ReadOnly
);

bool IsGPUSkinSceneExtensionEnabled()
{
	return GGPUSkinSceneExtensionEnabled;
}

static int32 GSkeletalMeshThrottleGPUSkinRayTracingUpdates = 0;
static FAutoConsoleVariableRef CVarSkeletalMeshThrottleGPUSkinRayTracingUpdates(
	TEXT("r.SkeletalMesh.ThrottleGPUSkinRayTracingUpdates"),
	GSkeletalMeshThrottleGPUSkinRayTracingUpdates,
	TEXT("Throttles the number of GPU Skin ray tracing GPU skin cache updates to N per frame (excluding required updates due to LOD changes)"),
	ECVF_RenderThreadSafe);

static bool UseGPUMorphTargets(ERHIFeatureLevel::Type FeatureLevel)
{
	return GUseGPUMorphTargets != 0 && FeatureLevel >= ERHIFeatureLevel::SM5;
}

static float GMorphTargetWeightThreshold = UE_SMALL_NUMBER;
static FAutoConsoleVariableRef CVarMorphTargetWeightThreshold(
	TEXT("r.MorphTarget.WeightThreshold"),
	GMorphTargetWeightThreshold,
	*FString::Printf(TEXT("Set MorphTarget Weight Threshold (Default : %f).\n"), UE_SMALL_NUMBER), 
	ECVF_Default
);

static int32 GetRayTracingSkeletalMeshGlobalLODBias()
{
	static const TConsoleVariableData<int32>* const RayTracingSkeletalMeshLODBiasVar =
		IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.RayTracing.Geometry.SkeletalMeshes.LODBias"));

	return !RayTracingSkeletalMeshLODBiasVar ? 0 :
		FMath::Max(0, RayTracingSkeletalMeshLODBiasVar->GetValueOnAnyThread());  // Only allows positive bias to narrow cloth mapping requirements
}

inline ESkeletalMeshGPUSkinTechnique GetGPUSkinTechnique(USkinnedMeshComponent* MeshComponent, FSkeletalMeshRenderData* RenderData, int32 LODIndex, ERHIFeatureLevel::Type FeatureLevel)
{
	ESkeletalMeshGPUSkinTechnique GPUSkinTechnique = ESkeletalMeshGPUSkinTechnique::Inline;

	if (MeshComponent)
	{
		if (MeshComponent->GetMeshDeformerInstanceForLOD(LODIndex) != nullptr)
		{
			GPUSkinTechnique = ESkeletalMeshGPUSkinTechnique::MeshDeformer;
		}
		else if (!AreInlineSkinningShadersEnabled(GetFeatureLevelShaderPlatform(FeatureLevel)) || MeshComponent->IsSkinCacheAllowed(LODIndex, FeatureLevel))
		{
			GPUSkinTechnique = ESkeletalMeshGPUSkinTechnique::GPUSkinCache;

			if (FeatureLevel == ERHIFeatureLevel::ES3_1)
			{
				// Some mobile GPUs (MALI) has a 64K elements limitation on texel buffers
				// SkinCache fetches mesh position through R32F texel buffer, thus any mesh that has more than 64K/3 vertices will not work correctly on such GPUs
				// We force this limitation for all mobile, to have an uniform behaviour across all mobile platforms
				if (RenderData->LODRenderData[LODIndex].GetNumVertices() * 3 >= (64 * 1024))
				{
					GPUSkinTechnique = ESkeletalMeshGPUSkinTechnique::Inline;
				}
			}
		}
	}

	return GPUSkinTechnique;
}

inline ESkeletalMeshGPUSkinTechnique GetGPUSkinTechnique(const FSkinnedMeshSceneProxyDesc& InMeshDesc, FSkeletalMeshRenderData* RenderData, int32 LODIndex, ERHIFeatureLevel::Type FeatureLevel)
{
	ESkeletalMeshGPUSkinTechnique GPUSkinTechnique = ESkeletalMeshGPUSkinTechnique::Inline;

	if (InMeshDesc.GetMeshDeformerInstanceForLOD(LODIndex) != nullptr)
	{
		GPUSkinTechnique = ESkeletalMeshGPUSkinTechnique::MeshDeformer;
	}
	else if (!AreInlineSkinningShadersEnabled(GetFeatureLevelShaderPlatform(FeatureLevel)) || InMeshDesc.IsSkinCacheAllowed(LODIndex))
	{
		GPUSkinTechnique = ESkeletalMeshGPUSkinTechnique::GPUSkinCache;

		if (FeatureLevel == ERHIFeatureLevel::ES3_1)
		{
			// Some mobile GPUs (MALI) has a 64K elements limitation on texel buffers
			// SkinCache fetches mesh position through R32F texel buffer, thus any mesh that has more than 64K/3 vertices will not work correctly on such GPUs
			// We force this limitation for all mobile, to have an uniform behaviour across all mobile platforms
			if (RenderData->LODRenderData[LODIndex].GetNumVertices() * 3 >= (64 * 1024))
			{
				GPUSkinTechnique = ESkeletalMeshGPUSkinTechnique::Inline;
			}
		}
	}

	return GPUSkinTechnique;
}
/*-----------------------------------------------------------------------------
FMorphVertexBuffer
-----------------------------------------------------------------------------*/

void FMorphVertexBuffer::InitRHI(FRHICommandListBase& RHICmdList)
{
	// LOD of the skel mesh is used to find number of vertices in buffer
	FSkeletalMeshLODRenderData& LodData = SkelMeshRenderData->LODRenderData[LODIdx];

	const bool bUseGPUMorphTargets = UseGPUMorphTargets(FeatureLevel);
	bUsesComputeShader = bUseGPUMorphTargets;

	const FRHIBufferCreateDesc CreateDesc =
		FRHIBufferCreateDesc::CreateVertex<FMorphGPUSkinVertex>(TEXT("MorphVertexBuffer"), LodData.GetNumVertices())
		// EBufferUsageFlags::ShaderResource is needed for Morph support of the SkinCache
		.AddUsage(EBufferUsageFlags::ShaderResource)
		.AddUsage(bUseGPUMorphTargets ? (EBufferUsageFlags)(EBufferUsageFlags::Static | EBufferUsageFlags::UnorderedAccess) : EBufferUsageFlags::Dynamic)
		.SetOwnerName(GetOwnerName())
		.SetInitialState(ERHIAccess::UAVCompute);

	if (!bUseGPUMorphTargets)
	{
		VertexBufferRHI = UE::RHIResourceUtils::CreateBufferZeroed(RHICmdList, CreateDesc);
		bNeedsInitialClear = false;
	}
	else
	{
		VertexBufferRHI = RHICmdList.CreateBuffer(CreateDesc);
		UAVValue = RHICmdList.CreateUnorderedAccessView(
			VertexBufferRHI, 
			FRHIViewDesc::CreateBufferUAV()
				.SetType(FRHIViewDesc::EBufferType::Typed)
				.SetFormat(PF_R32_UINT));
		bNeedsInitialClear = true;
	}

	SRVValue = RHICmdList.CreateShaderResourceView(
		VertexBufferRHI, 
		FRHIViewDesc::CreateBufferSRV()
			.SetType(FRHIViewDesc::EBufferType::Typed)
			.SetFormat(PF_R32_FLOAT));

	// hasn't been updated yet
	bHasBeenUpdated = false;
}

void FMorphVertexBuffer::ReleaseRHI()
{
	UAVValue.SafeRelease();
	VertexBufferRHI.SafeRelease();
	SRVValue.SafeRelease();
}

/*-----------------------------------------------------------------------------
FMorphVertexBufferPool
-----------------------------------------------------------------------------*/
void FMorphVertexBufferPool::InitResources(FName OwnerName)
{
	// InitResources may be called again when morph vertex data is persisted during render state re-creation.
	if (!bInitializedResources)
	{
		check(!MorphVertexBuffers[0].VertexBufferRHI.IsValid());
		check(!MorphVertexBuffers[1].VertexBufferRHI.IsValid());
		BeginInitResource(OwnerName, &MorphVertexBuffers[0], &UE::RenderCommandPipe::SkeletalMesh);
		if (bDoubleBuffer)
		{
			BeginInitResource(OwnerName, &MorphVertexBuffers[1], &UE::RenderCommandPipe::SkeletalMesh);
		}

		bInitializedResources = true;
	}
}

void FMorphVertexBufferPool::ReleaseResources()
{
	check(bInitializedResources);
	MorphVertexBuffers[0].ReleaseResource();
	MorphVertexBuffers[1].ReleaseResource();
	bInitializedResources = false;
}

SIZE_T FMorphVertexBufferPool::GetResourceSize() const
{
	SIZE_T ResourceSize = sizeof(*this);
	ResourceSize += MorphVertexBuffers[0].GetResourceSize();
	ResourceSize += MorphVertexBuffers[1].GetResourceSize();
	return ResourceSize;
}

void FMorphVertexBufferPool::EnableDoubleBuffer(FRHICommandListBase& RHICmdList)
{
	bDoubleBuffer = true;
	if (!MorphVertexBuffers[1].VertexBufferRHI.IsValid())
	{
		MorphVertexBuffers[1].InitResource(RHICmdList);
	}
}

void FMorphVertexBufferPool::SetCurrentRevisionNumber(uint32 RevisionNumber)
{
	if (bDoubleBuffer)
	{
		// Flip revision number to previous if this is new, otherwise keep current version.
		if (CurrentRevisionNumber != RevisionNumber)
		{
			PreviousRevisionNumber = CurrentRevisionNumber;
			CurrentRevisionNumber = RevisionNumber;
			CurrentBuffer = 1 - CurrentBuffer;
		}
	}
}

uint32 FMorphVertexBufferPool::GetReadingIndex(bool bPrevious) const
{
	uint32 Index = 0;
	if (bDoubleBuffer)
	{
		if ((CurrentRevisionNumber - PreviousRevisionNumber) > 1)
		{
			// If the revision number has incremented too much, ignore the request and use the current buffer.
			// With ClearMotionVector calls, we intentionally increment revision number to retrieve current buffer for bPrevious true.
			bPrevious = false;
		}

		Index = CurrentBuffer ^ (uint32)bPrevious;

		if (!MorphVertexBuffers[Index].bHasBeenUpdated)
		{
			// this should only happen the first time updating, in which case the previous buffer hasn't been written into yet.
			check(Index == 1);
			Index = 0;
		}
	}
	return Index;
}

const FMorphVertexBuffer& FMorphVertexBufferPool::GetMorphVertexBufferForReading(bool bPrevious) const
{
	const uint32 Index = GetReadingIndex(bPrevious);
	checkf(MorphVertexBuffers[Index].VertexBufferRHI.IsValid(), TEXT("Index: %i Buffer0: %s Buffer1: %s"), Index, MorphVertexBuffers[0].VertexBufferRHI.IsValid() ? TEXT("true") : TEXT("false"), MorphVertexBuffers[1].VertexBufferRHI.IsValid() ? TEXT("true") : TEXT("false"));
	return MorphVertexBuffers[Index];
}

const FMorphVertexBuffer* FMorphVertexBufferPool::TryGetMorphVertexBufferForReading(bool bPrevious) const
{
	const uint32 Index = GetReadingIndex(bPrevious);
	if (!MorphVertexBuffers[Index].VertexBufferRHI.IsValid())
	{
		return nullptr;
	}
	return &MorphVertexBuffers[Index];
}

FMorphVertexBuffer& FMorphVertexBufferPool::GetMorphVertexBufferForWriting()
{
	return MorphVertexBuffers[CurrentBuffer];
}

///////////////////////////////////////////////////////////////////////////////

class FSkeletalMeshUpdatePacketGPUSkin final : public TSkeletalMeshUpdatePacket<FSkeletalMeshObjectGPUSkin, FDynamicSkelMeshObjectDataGPUSkin>
{
public:
	// Template Overrides
	void Add(FSkeletalMeshObjectGPUSkin* MeshObject, FDynamicSkelMeshObjectDataGPUSkin* DynamicData);
	void Free(FDynamicSkelMeshObjectDataGPUSkin* DynamicData);

	// Virtual Overrides
	void Init(const FInitializer& Initializer) override;
	void ProcessStage_MeshDeformer(FRHICommandList& RHICmdList, UE::Tasks::FTaskEvent& TaskEvent) override;
	void ProcessStage_Inline(FRHICommandList& RHICmdList, UE::Tasks::FTaskEvent& TaskEvent) override;
	void ProcessStage_SkinCache(FRHICommandList& RHICmdList, UE::Tasks::FTaskEvent& TaskEvent) override;
	void ProcessStage_Upload(FRHICommandList& RHICmdList) override;

private:
	FGPUBaseSkinVertexFactory::FUpdateScope UpdateScope;

	struct FCommand
	{
		FCommand(FSkeletalMeshObjectGPUSkin* InMeshObject)
			: MeshObject(InMeshObject)
		{}

		FSkeletalMeshObjectGPUSkin* MeshObject;
		bool bUpdateRayTracingMode = false;
	};

	void ProcessUpdatedDynamicData(TArrayView<FCommand> Commands, FRHICommandList& RHICmdList);
	void UpdateBufferData(TConstArrayView<FCommand> Commands, FRHICommandList& RHICmdList);

	TArray<FDynamicSkelMeshObjectDataGPUSkin*, FConcurrentLinearArrayAllocator> DynamicDatasToFree;
	TArray<FSkeletalMeshObjectGPUSkin*, FConcurrentLinearArrayAllocator> MeshDeformer;
	TArray<FCommand, FConcurrentLinearArrayAllocator> Inline;
	TArray<FCommand, FConcurrentLinearArrayAllocator> SkinCache;
};

REGISTER_SKELETAL_MESH_UPDATE_BACKEND(FSkeletalMeshUpdatePacketGPUSkin);

void FSkeletalMeshUpdatePacketGPUSkin::Add(FSkeletalMeshObjectGPUSkin* MeshObject, FDynamicSkelMeshObjectDataGPUSkin* DynamicData)
{
	MeshObject->bMorphNeedsUpdate = FDynamicSkelMeshObjectDataGPUSkin::IsMorphUpdateNeeded(MeshObject->DynamicData, DynamicData);

	if (MeshObject->PreviousDynamicData)
	{
		DynamicDatasToFree.Emplace(MeshObject->PreviousDynamicData);
	}

	MeshObject->PreviousDynamicData = MeshObject->DynamicData;
	MeshObject->DynamicData = DynamicData;

#if RHI_RAYTRACING
	if (MeshObject->bMorphNeedsUpdate)
	{
		InvalidatePathTracedOutput();
	}
#endif

	switch (DynamicData->GPUSkinTechnique)
	{
	case ESkeletalMeshGPUSkinTechnique::Inline:
		Inline.Emplace(MeshObject);
		break;

	case ESkeletalMeshGPUSkinTechnique::GPUSkinCache:
		SkinCache.Emplace(MeshObject);
		break;

	case ESkeletalMeshGPUSkinTechnique::MeshDeformer:
		MeshDeformer.Emplace(MeshObject);
		break;
	}
}

void FSkeletalMeshUpdatePacketGPUSkin::Free(FDynamicSkelMeshObjectDataGPUSkin* DynamicData)
{
	DynamicDatasToFree.Emplace(DynamicData);
}

void FSkeletalMeshUpdatePacketGPUSkin::Init(const FInitializer& Initializer)
{
	DynamicDatasToFree.Reserve(Initializer.NumRemoves);
	MeshDeformer.Reserve(Initializer.NumUpdates);
	SkinCache.Reserve(Initializer.NumUpdates);
	Inline.Reserve(Initializer.NumUpdates);
}

void FSkeletalMeshUpdatePacketGPUSkin::ProcessStage_MeshDeformer(FRHICommandList& RHICmdList, UE::Tasks::FTaskEvent& TaskEvent)
{
	for (FSkeletalMeshObjectGPUSkin* MeshObject : MeshDeformer)
	{
		MeshObject->ProcessUpdatedDynamicData(RHICmdList, EGPUSkinCacheEntryMode::Raster);
	}
}

void FSkeletalMeshUpdatePacketGPUSkin::ProcessStage_SkinCache(FRHICommandList& RHICmdList, UE::Tasks::FTaskEvent& TaskEvent)
{
	ProcessUpdatedDynamicData(SkinCache, RHICmdList);
}

void FSkeletalMeshUpdatePacketGPUSkin::ProcessStage_Inline(FRHICommandList& RHICmdList, UE::Tasks::FTaskEvent& TaskEvent)
{
	ProcessUpdatedDynamicData(Inline, RHICmdList);
}

void FSkeletalMeshUpdatePacketGPUSkin::ProcessStage_Upload(FRHICommandList& RHICmdList)
{
	for (FSkeletalMeshObjectGPUSkin* MeshObject : MeshDeformer)
	{
		MeshObject->UpdateBufferData(RHICmdList, EGPUSkinCacheEntryMode::Raster);
	}

	UpdateBufferData(Inline, RHICmdList);

	{
		FRHICommandListScopedPipeline SkinCacheScope(RHICmdList, GPUSkinCachePipeline);
		UpdateBufferData(SkinCache, RHICmdList);
	}

	for (FDynamicSkelMeshObjectDataGPUSkin* DynamicData : DynamicDatasToFree)
	{
		FDynamicSkelMeshObjectDataGPUSkin::Release(DynamicData);
	}
}

void FSkeletalMeshUpdatePacketGPUSkin::ProcessUpdatedDynamicData(TArrayView<FCommand> Commands, FRHICommandList& RHICmdList)
{
#if RHI_RAYTRACING
	struct FSkinCacheEntryThrottled
	{
		FSkeletalMeshObjectGPUSkin* MeshObject;
		uint32 BoneFrameDelta;
		uint32 CommandIndex;
	};

	TArray<FSkinCacheEntryThrottled, FConcurrentLinearArrayAllocator> SkinCacheRayTracingThrottled;

	if (GSkeletalMeshThrottleGPUSkinRayTracingUpdates > 0)
	{
		SkinCacheRayTracingThrottled.Reserve(Commands.Num());
	}
#endif

	for (int32 Index = 0; Index < Commands.Num(); ++Index)
	{
		FCommand& Command = Commands[Index];
		FSkeletalMeshObjectGPUSkin& MeshObject = *Command.MeshObject;
		MeshObject.ProcessUpdatedDynamicData(RHICmdList, EGPUSkinCacheEntryMode::Raster);

	#if RHI_RAYTRACING
		if (IsSkinCacheForRayTracingSupported() && MeshObject.IsRayTracingSkinCacheUpdateNeeded())
		{
			FDynamicSkelMeshObjectDataGPUSkin* DynamicData = MeshObject.DynamicData;

			if (DynamicData->IsRequiredUpdate() || GSkeletalMeshThrottleGPUSkinRayTracingUpdates <= 0)
			{
				MeshObject.ProcessUpdatedDynamicData(RHICmdList, EGPUSkinCacheEntryMode::RayTracing);
				MeshObject.LastRayTracingBoneTransformUpdate = DynamicData->BoneTransformFrameNumber;
				Command.bUpdateRayTracingMode = true;
			}
			else
			{
				SkinCacheRayTracingThrottled.Emplace(FSkinCacheEntryThrottled{ &MeshObject, DynamicData->BoneTransformFrameNumber - MeshObject.LastRayTracingBoneTransformUpdate, (uint32)Index });
			}
		}
		else
		{
			FGPUSkinCache::Release(MeshObject.SkinCacheEntryForRayTracing);
		}

		if (!MeshObject.GetSkinCacheEntryForRayTracing())
		{
			// When SkinCacheEntry is gone, clear geometry
			MeshObject.RayTracingGeometry.ReleaseRHI();
			MeshObject.RayTracingGeometry.SetInitializer(FRayTracingGeometryInitializer{});
		}

		if (!SkinCacheRayTracingThrottled.IsEmpty())
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(SkinCacheRayTracingThrottled);
			FSkinCacheEntryThrottled* StartIt = SkinCacheRayTracingThrottled.GetData();
			FSkinCacheEntryThrottled* EndIt   = SkinCacheRayTracingThrottled.GetData() + SkinCacheRayTracingThrottled.Num();
			FSkinCacheEntryThrottled* SortIt  = SkinCacheRayTracingThrottled.GetData() + FMath::Min(GSkeletalMeshThrottleGPUSkinRayTracingUpdates, SkinCacheRayTracingThrottled.Num());

			std::partial_sort(StartIt, SortIt, EndIt, [] (FSkinCacheEntryThrottled A, FSkinCacheEntryThrottled B)
			{
				return A.BoneFrameDelta > B.BoneFrameDelta;
			});

			if (SkinCacheRayTracingThrottled.Num() > GSkeletalMeshThrottleGPUSkinRayTracingUpdates)
			{
				SkinCacheRayTracingThrottled.SetNum(GSkeletalMeshThrottleGPUSkinRayTracingUpdates, EAllowShrinking::No);
			}
		}

		for (FSkinCacheEntryThrottled Entry : SkinCacheRayTracingThrottled)
		{
			Entry.MeshObject->ProcessUpdatedDynamicData(RHICmdList, EGPUSkinCacheEntryMode::RayTracing);
			Entry.MeshObject->LastRayTracingBoneTransformUpdate = Entry.MeshObject->DynamicData->BoneTransformFrameNumber;
			Commands[Entry.CommandIndex].bUpdateRayTracingMode = true;
		}
	#endif
	}
}

void FSkeletalMeshUpdatePacketGPUSkin::UpdateBufferData(TConstArrayView<FCommand> Commands, FRHICommandList& RHICmdList)
{
	for (const FCommand& Command : Commands)
	{
		FSkeletalMeshObjectGPUSkin& MeshObject = *Command.MeshObject;
		MeshObject.UpdateBufferData(RHICmdList, EGPUSkinCacheEntryMode::Raster);
		if (Command.bUpdateRayTracingMode)
		{
			MeshObject.UpdateBufferData(RHICmdList, EGPUSkinCacheEntryMode::RayTracing);
		}
	}
}

///////////////////////////////////////////////////////////////////////////////

FSkeletalMeshObjectGPUSkin::FSkeletalMeshObjectGPUSkin(const USkinnedMeshComponent* InMeshComponent, FSkeletalMeshRenderData* InSkelMeshRenderData, ERHIFeatureLevel::Type InFeatureLevel)
	: FSkeletalMeshObjectGPUSkin(FSkinnedMeshSceneProxyDesc(InMeshComponent), InSkelMeshRenderData, InFeatureLevel) {}

FSkeletalMeshObjectGPUSkin::FSkeletalMeshObjectGPUSkin(const FSkinnedMeshSceneProxyDesc& InMeshDesc, FSkeletalMeshRenderData* InSkelMeshRenderData, ERHIFeatureLevel::Type InFeatureLevel)
	: FSkeletalMeshObject(InMeshDesc, InSkelMeshRenderData, InFeatureLevel)
{
	BoneTransformStorageMode = EBoneTransformStorageMode::BoneMap;

	FSkeletalMeshObjectGPUSkin* PreviousMeshObject = nullptr;
	if (InMeshDesc.PreviousMeshObject && InMeshDesc.PreviousMeshObject->IsGPUSkinMesh())
	{
		PreviousMeshObject = (FSkeletalMeshObjectGPUSkin*)InMeshDesc.PreviousMeshObject;

		// Don't use re-create data if the mesh or feature level changed
		if (PreviousMeshObject->SkeletalMeshRenderData != InSkelMeshRenderData || PreviousMeshObject->FeatureLevel != InFeatureLevel)
		{
			PreviousMeshObject = nullptr;
		}
	}

	if (PreviousMeshObject)
	{
		// Transfer GPU skin cache from PreviousMeshObject -- needs to happen on render thread.  PreviousMeshObject is defer deleted, so it's safe to access it there.
		ENQUEUE_RENDER_COMMAND(ReleaseSkeletalMeshSkinCacheResources)(UE::RenderCommandPipe::SkeletalMesh,
			[this, PreviousMeshObject](FRHICommandList& RHICmdList)
			{
				SkinCacheEntry = PreviousMeshObject->SkinCacheEntry;
				SkinCacheEntryForRayTracing = PreviousMeshObject->SkinCacheEntryForRayTracing;

				// patch entries to point to new GPUSkin
				FGPUSkinCache::SetEntryGPUSkin(SkinCacheEntry, this);
				FGPUSkinCache::SetEntryGPUSkin(SkinCacheEntryForRayTracing, this);

				PreviousMeshObject->SkinCacheEntry = nullptr;
				PreviousMeshObject->SkinCacheEntryForRayTracing = nullptr;
			}
		);
	}

	BoneTransformAllocationMode = ::GetBoneTransformAllocationMode();

	// create LODs to match the base mesh
	LODs.Empty(SkeletalMeshRenderData->LODRenderData.Num());
	for (int32 LODIndex=0; LODIndex < SkeletalMeshRenderData->LODRenderData.Num(); LODIndex++)
	{
		FMorphVertexBufferPool* RecreateMorphVertexBuffer = nullptr;
		if (PreviousMeshObject)
		{
			RecreateMorphVertexBuffer = PreviousMeshObject->LODs[LODIndex].MorphVertexBufferPool;
		}

		const ESkeletalMeshGPUSkinTechnique GPUSkinTechnique = ::GetGPUSkinTechnique(InMeshDesc, SkeletalMeshRenderData, LODIndex, InFeatureLevel);

		FSkeletalMeshObjectLOD& LOD = LODs.Emplace_GetRef(SkeletalMeshRenderData, LODIndex, InFeatureLevel, RecreateMorphVertexBuffer, GPUSkinTechnique);

		bool bHasValidSections = false;

		for (const FSkelMeshRenderSection& Section : GetRenderSections(LODIndex))
		{
			if (Section.IsValid())
			{
				bHasValidSections = true;
				break;
			}
		}

		// Fall back to vertex factory buffer allocation mode if any of the LOD's require it or if an LOD has no valid sections in it.
		if (!bHasValidSections || GPUSkinTechnique != ESkeletalMeshGPUSkinTechnique::Inline)
		{
			BoneTransformAllocationMode = EBoneTransformAllocationMode::VertexFactory;
		}
	}

	InitResources(InMeshDesc);
}


FSkeletalMeshObjectGPUSkin::~FSkeletalMeshObjectGPUSkin()
{
	FDynamicSkelMeshObjectDataGPUSkin::Release(DynamicData);
	FDynamicSkelMeshObjectDataGPUSkin::Release(PreviousDynamicData);
	DynamicData = nullptr;
	PreviousDynamicData = nullptr;
}

void FSkeletalMeshObjectGPUSkin::InitResources(const FSkinnedMeshSceneProxyDesc& InMeshDesc)
{
	checkf(InMeshDesc.Scene && InMeshDesc.Scene->GetSkeletalMeshUpdater(), TEXT("Scene cannot be null when creating a SkeletalMeshObject."));
	UpdateHandle = InMeshDesc.Scene->GetSkeletalMeshUpdater()->Create(this);
	GPUSkinCache = InMeshDesc.Scene->GetGPUSkinCache();

	for( int32 LODIndex=0;LODIndex < LODs.Num();LODIndex++ )
	{
		FSkeletalMeshObjectLOD& SkelLOD = LODs[LODIndex];

		// Skip LODs that have their render data stripped
		if (SkelLOD.SkelMeshRenderData && SkelLOD.SkelMeshRenderData->LODRenderData.IsValidIndex(LODIndex) && SkelLOD.SkelMeshRenderData->LODRenderData[LODIndex].GetNumVertices() > 0)
		{
			const FSkelMeshObjectLODInfo& MeshLODInfo = LODInfo[LODIndex];

			const FSkelMeshComponentLODInfo* CompLODInfo =  (InMeshDesc.LODInfo.IsValidIndex(LODIndex)) ? &InMeshDesc.LODInfo[LODIndex] : nullptr;

			FGPUSkinPassthroughVertexFactory::EVertexAttributeFlags VertexAttributeMask = FGPUSkinPassthroughVertexFactory::EVertexAttributeFlags::None;

			if (SkelLOD.GPUSkinTechnique == ESkeletalMeshGPUSkinTechnique::MeshDeformer)
			{
				const EMeshDeformerOutputBuffer OutputBuffers = InMeshDesc.GetMeshDeformerInstanceForLOD(LODIndex)->GetOutputBuffers();

				if (EnumHasAnyFlags(OutputBuffers, EMeshDeformerOutputBuffer::SkinnedMeshPosition))
				{
					VertexAttributeMask |= FGPUSkinPassthroughVertexFactory::EVertexAttributeFlags::Position;
				}

				if (EnumHasAnyFlags(OutputBuffers, EMeshDeformerOutputBuffer::SkinnedMeshVertexColor))
				{
					VertexAttributeMask |= FGPUSkinPassthroughVertexFactory::EVertexAttributeFlags::Color;
				}

				if (EnumHasAnyFlags(OutputBuffers, EMeshDeformerOutputBuffer::SkinnedMeshTangents))
				{
					VertexAttributeMask |= FGPUSkinPassthroughVertexFactory::EVertexAttributeFlags::Tangent;
				}
			}
			else if (SkelLOD.GPUSkinTechnique == ESkeletalMeshGPUSkinTechnique::GPUSkinCache || (FGPUSkinCache::IsGPUSkinCacheRayTracingSupported(FeatureLevel) && SkelLOD.SkelMeshRenderData->bSupportRayTracing))
			{
				VertexAttributeMask = FGPUSkinPassthroughVertexFactory::EVertexAttributeFlags::Position | FGPUSkinPassthroughVertexFactory::EVertexAttributeFlags::Tangent;
			}

			SkelLOD.InitResources(GPUSkinCache, MeshLODInfo, CompLODInfo, FeatureLevel, VertexAttributeMask, BoneTransformStorageMode, BoneTransformAllocationMode);
		}
	}

#if RHI_RAYTRACING
	if (IsRayTracingEnabled() && bSupportRayTracing)
	{
		BeginInitResource(&RayTracingGeometry, &UE::RenderCommandPipe::SkeletalMesh);
	}
#endif
}
void FSkeletalMeshObjectGPUSkin::ReleaseResources()
{
	UpdateHandle.Release();

	for ( int32 LODIndex=0;LODIndex < LODs.Num();LODIndex++ )
	{
		FSkeletalMeshObjectLOD& SkelLOD = LODs[LODIndex];
		SkelLOD.ReleaseResources(GPUSkinCache);
	}
	// also release morph resources
	FSkeletalMeshObjectGPUSkin* MeshObject = this;
	FGPUSkinCacheEntry** PtrSkinCacheEntry = &SkinCacheEntry;
	ENQUEUE_RENDER_COMMAND(ReleaseSkeletalMeshSkinCacheResources)(UE::RenderCommandPipe::SkeletalMesh,
		[MeshObject, PtrSkinCacheEntry, &SkinCacheEntryForRayTracing = SkinCacheEntryForRayTracing](FRHICommandList& RHICmdList)
		{
			FGPUSkinCacheEntry*& LocalSkinCacheEntry = *PtrSkinCacheEntry;
			FGPUSkinCache::Release(LocalSkinCacheEntry);
			FGPUSkinCacheEntry* LocalSkinCacheEntryForRayTracing = SkinCacheEntryForRayTracing;
			FGPUSkinCache::Release(LocalSkinCacheEntryForRayTracing);

			*PtrSkinCacheEntry = nullptr;
			SkinCacheEntryForRayTracing = nullptr;
		}
	);

#if RHI_RAYTRACING
	if (bSupportRayTracing)
	{
		BeginReleaseResource(&RayTracingGeometry, &UE::RenderCommandPipe::SkeletalMesh);
	}
#endif // RHI_RAYTRACING
}

void FSkeletalMeshObjectGPUSkin::InitMorphResources()
{
	if (!bMorphResourcesInitialized)
	{
		for( int32 LODIndex=0;LODIndex < LODs.Num();LODIndex++ )
		{
			FSkeletalMeshObjectLOD& SkelLOD = LODs[LODIndex];

			// Check the LOD render data for verts, if it's been stripped we don't create morph buffers
			const int32 LodIndexInMesh = SkelLOD.LODIndex;
			const FSkeletalMeshLODRenderData& RenderData = SkelLOD.SkelMeshRenderData->LODRenderData[LodIndexInMesh];

			if(RenderData.GetNumVertices() > 0)
			{
				// init any morph vertex buffers for each LOD
				const FSkelMeshObjectLODInfo& MeshLODInfo = LODInfo[LODIndex];
				SkelLOD.InitMorphResources(MeshLODInfo, FeatureLevel);
			}
		}
		bMorphResourcesInitialized = true;
	}
}

void FSkeletalMeshObjectGPUSkin::Update(
	int32 LODIndex,
	const FSkinnedMeshSceneProxyDynamicData& InDynamicData,
	const FPrimitiveSceneProxy* InSceneProxy,
	const USkinnedAsset* InSkinnedAsset,
	const FMorphTargetWeightMap& InActiveMorphTargets,
	const TArray<float>& InMorphTargetWeights,
	EPreviousBoneTransformUpdateMode PreviousBoneTransformUpdateMode,
	const FExternalMorphWeightData& InExternalMorphWeightData)
{
	LLM_SCOPE(ELLMTag::SkeletalMesh);

	// make sure morph data has been initialized for each LOD
	if((!InActiveMorphTargets.IsEmpty() || !InExternalMorphWeightData.MorphSets.IsEmpty()))
	{
		// initialized on-the-fly in order to avoid creating extra vertex streams for each skel mesh instance
		InitMorphResources();
	}

	// create the new dynamic data for use by the rendering thread
	// this data is only deleted when another update is sent
	const int32 NumBones = InDynamicData.bUseReferenceToLocalMatrices ? InDynamicData.ReferenceToLocalMatrices.Num() : InDynamicData.ComponentSpaceTransforms.Num();
	FDynamicSkelMeshObjectDataGPUSkin* NewDynamicData = FDynamicSkelMeshObjectDataGPUSkin::Acquire(NumBones);
	NewDynamicData->InitDynamicSkelMeshObjectDataGPUSkin(
		InDynamicData,
		InSceneProxy,
		InSkinnedAsset,
		SkeletalMeshRenderData,
		this,
		LODIndex,
		InActiveMorphTargets,
		InMorphTargetWeights,
		PreviousBoneTransformUpdateMode,
		InExternalMorphWeightData);

	UpdateHandle.Update(NewDynamicData);
}

void FSkeletalMeshObjectGPUSkin::UpdateSkinWeightBuffer(const TArrayView<const FSkelMeshComponentLODInfo> InLODInfo)
{
	for (int32 LODIndex = 0; LODIndex < LODs.Num(); LODIndex++)
	{
		FSkeletalMeshObjectLOD& SkelLOD = LODs[LODIndex];

		// Skip LODs that have their render data stripped
		if (SkelLOD.SkelMeshRenderData->LODRenderData[LODIndex].GetNumVertices() > 0)
		{
			const FSkelMeshComponentLODInfo* CompLODInfo = InLODInfo.IsValidIndex(LODIndex) ? &InLODInfo[LODIndex] : nullptr;

			SkelLOD.UpdateSkinWeights(CompLODInfo, BoneTransformStorageMode);
		}
	}
	
	ENQUEUE_RENDER_COMMAND(UpdateSkinCacheSkinWeightBuffer)(UE::RenderCommandPipe::SkeletalMesh,
		[this](FRHICommandList& RHICmdList)
	{
		if (SkinCacheEntry)
		{
			FGPUSkinCache::UpdateSkinWeightBuffer(SkinCacheEntry);
		}

		if (SkinCacheEntryForRayTracing)
		{
			FGPUSkinCache::UpdateSkinWeightBuffer(SkinCacheEntryForRayTracing);
		}
	});
}

bool FDynamicSkelMeshObjectDataGPUSkin::IsMorphUpdateNeeded(const FDynamicSkelMeshObjectDataGPUSkin* Previous, const FDynamicSkelMeshObjectDataGPUSkin* Current)
{
	if (!Previous)
	{
		return true;
	}

	// If external morphs are present, we update each time. Make sure to check the previous frame's morph as well 
	// or we can end up with stale render data.
	if (Current->ExternalMorphWeightData.HasActiveMorphs() || Previous->ExternalMorphWeightData.HasActiveMorphs())
	{
		return true;
	}

	return Previous->LODIndex != Current->LODIndex || !Previous->ActiveMorphTargetsEqual(Current->ActiveMorphTargets, Current->MorphTargetWeights);
}

TConstArrayView<FMatrix44f> FDynamicSkelMeshObjectDataGPUSkin::GetPreviousReferenceToLocal(EGPUSkinCacheEntryMode Mode) const
{
	return PreviousReferenceToLocal;
}

TConstArrayView<FMatrix44f> FDynamicSkelMeshObjectDataGPUSkin::GetReferenceToLocal(EGPUSkinCacheEntryMode Mode) const
{
#if RHI_RAYTRACING
	return Mode == EGPUSkinCacheEntryMode::RayTracing && RayTracingLODIndex != LODIndex
			? ReferenceToLocalForRayTracing
			: ReferenceToLocal;
#else
	return ReferenceToLocal;
#endif
}

int32 FDynamicSkelMeshObjectDataGPUSkin::GetLODIndex(EGPUSkinCacheEntryMode Mode) const
{
#if RHI_RAYTRACING
	return Mode == EGPUSkinCacheEntryMode::RayTracing ? RayTracingLODIndex : LODIndex;
#else
	return LODIndex;
#endif
}


bool FSkeletalMeshObjectGPUSkin::IsSkinCacheEnabled(EGPUSkinCacheEntryMode Mode) const
{
	return FGPUSkinCache::IsGPUSkinCacheEnabled(FeatureLevel) &&
		// Force skin cache enabled for ray tracing if the inline skinning technique was requested.
		(DynamicData->GPUSkinTechnique == ESkeletalMeshGPUSkinTechnique::GPUSkinCache || (DynamicData->GPUSkinTechnique == ESkeletalMeshGPUSkinTechnique::Inline && Mode == EGPUSkinCacheEntryMode::RayTracing));
}

void FSkeletalMeshObjectGPUSkin::UpdateBufferData(FRHICommandList& RHICmdList, EGPUSkinCacheEntryMode Mode)
{
	const int32 LODIndex = DynamicData->GetLODIndex(Mode);
	FSkeletalMeshObjectLOD& LOD = LODs[LODIndex];

	if (Mode == EGPUSkinCacheEntryMode::Raster)
	{
		UpdateMorphVertexBuffer(RHICmdList);
	}

	if (BoneTransformAllocationMode == EBoneTransformAllocationMode::VertexFactory)
	{
		auto& VertexFactories = LOD.GPUSkinVertexFactories.VertexFactories;
		const TArray<FSkelMeshRenderSection>& Sections = GetRenderSections(LODIndex);
		const FName OwnerName = GetAssetPathName(LODIndex);

		TConstArrayView<FMatrix44f> PreviousReferenceToLocalMatrices;
		TConstArrayView<FMatrix44f> ReferenceToLocalMatrices = DynamicData->GetReferenceToLocal(Mode);

		if (Mode == EGPUSkinCacheEntryMode::Raster && DynamicData->PreviousBoneTransformUpdateMode != EPreviousBoneTransformUpdateMode::None)
		{
			PreviousReferenceToLocalMatrices = DynamicData->GetPreviousReferenceToLocal(Mode);
		}

		if (LOD.UnifiedBoneBuffer)
		{
			if (!PreviousReferenceToLocalMatrices.IsEmpty())
			{
				const bool bPrevious = true;
				LOD.UnifiedBoneBuffer->Update(RHICmdList, OwnerName, PreviousReferenceToLocalMatrices, bPrevious);
			}

			{
				const bool bPrevious = false;
				LOD.UnifiedBoneBuffer->Update(RHICmdList, OwnerName, ReferenceToLocalMatrices, bPrevious);
			}
		}

		for (int32 SectionIdx = 0; SectionIdx < Sections.Num(); SectionIdx++)
		{
			const FSkelMeshRenderSection& Section = Sections[SectionIdx];

			if (!Section.IsValid())
			{
				continue;
			}

			FGPUBaseSkinVertexFactory* VertexFactory = VertexFactories[SectionIdx].Get();

			if (!LOD.UnifiedBoneBuffer)
			{
				FGPUBaseSkinVertexFactory::FShaderDataType& ShaderData = VertexFactory->GetShaderData();

				if (!PreviousReferenceToLocalMatrices.IsEmpty())
				{
					const bool bPrevious = true;
					ShaderData.UpdateBoneData(RHICmdList, OwnerName, PreviousReferenceToLocalMatrices, Section.BoneMap, ShaderData.GetBoneBufferForWriting(bPrevious).VertexBufferRHI);
				}

				{
					const bool bPrevious = false;
					ShaderData.UpdateBoneData(RHICmdList, OwnerName, ReferenceToLocalMatrices, Section.BoneMap, ShaderData.GetBoneBufferForWriting(bPrevious).VertexBufferRHI);
				}
			}

			if (VertexFactory->IsUniformBufferValid())
			{
				VertexFactory->UpdateUniformBuffer(RHICmdList, nullptr, nullptr);
			}
		}
	}
}

void FSkeletalMeshObjectGPUSkin::ProcessUpdatedDynamicData(FRHICommandList& RHICmdList, EGPUSkinCacheEntryMode Mode)
{
	const int32 LODIndex = DynamicData->GetLODIndex(Mode);
	const uint32 BoneTransformFrameNumber = DynamicData->BoneTransformFrameNumber;
	const uint32 CurrentRevisionNumber = DynamicData->RevisionNumber;

	// Invalidate cached scene extension bone data until next call to UpdateSceneExtensionHeader.
	SceneExtensionBoneBuffer = {};

	FSkeletalMeshObjectLOD& LOD = LODs[LODIndex];
	FVertexFactoryData& VertexFactoryData = LOD.GPUSkinVertexFactories;
	const FSkeletalMeshLODRenderData& LODData = SkeletalMeshRenderData->LODRenderData[LODIndex];
	const TArray<FSkelMeshRenderSection>& Sections = GetRenderSections(LODIndex);
	const FName OwnerName = GetAssetPathName(LODIndex);

	const bool bHasWeightedActiveMorphs = DynamicData->NumWeightedActiveMorphTargets > 0;
	const bool bHasExternalMorphs = DynamicData->ExternalMorphWeightData.HasActiveMorphs() && !DynamicData->ExternalMorphWeightData.MorphSets.IsEmpty();
	
	const EShaderPlatform ShaderPlatform = GetFeatureLevelShaderPlatform(FeatureLevel);
	const bool bIsMobile = IsMobilePlatform(ShaderPlatform);

	bool bHasPreviousReferenceToLocal = false;
	int32 PreviousRevisionNumber = CurrentRevisionNumber;

	if (Mode == EGPUSkinCacheEntryMode::Raster)
	{
		// RayTracing does not need the previous buffer at all, so don't allocate it.
		bHasPreviousReferenceToLocal = DynamicData->PreviousBoneTransformUpdateMode != EPreviousBoneTransformUpdateMode::None;
		PreviousRevisionNumber = bHasPreviousReferenceToLocal ? DynamicData->PreviousRevisionNumber : INDEX_NONE;

		// RayTracing-specific LOD's can't have a separate morph target buffer at the moment because there is only one morph vertex buffer across the entire mesh.
		if (GEnableMorphTargets && LODData.GetNumVertices() > 0)
		{
			if (bHasWeightedActiveMorphs || bHasExternalMorphs)
			{
				bMorphNeedsUpdate |= GForceUpdateMorphTargets != 0;

				if (bMorphNeedsUpdate)
				{
					LOD.MorphVertexBufferPool->SetCurrentRevisionNumber(CurrentRevisionNumber);
				}

				MorphVertexBuffer = &LOD.MorphVertexBufferPool->GetMorphVertexBufferForWriting();
				MorphVertexBuffer->SectionIds = DynamicData->SectionIdsUseByActiveMorphTargets;

				// Force an update if this is the first use of the buffer
				if (!MorphVertexBuffer->bHasBeenUpdated)
				{
					bMorphNeedsUpdate = true;
				}

				if (bMorphNeedsUpdate)
				{
					LOD.MorphVertexBufferPool->SetUpdatedFrameNumber(BoneTransformFrameNumber);
				}
			}
			// The update is still set to true when transitioning from morphs to no-morphs. We need to clear out the previous buffer contents.
			else if (bMorphNeedsUpdate && LOD.MorphVertexBufferPool->IsInitialized())
			{
				MorphVertexBuffer = &LOD.MorphVertexBufferPool->GetMorphVertexBufferForWriting();
				MorphVertexBuffer->bNeedsInitialClear = true;
				bMorphNeedsUpdate = false;
			}
			else
			{
				MorphVertexBuffer = nullptr;
				bMorphNeedsUpdate = false;
			}
		}
		else
		{
			MorphVertexBuffer = nullptr;
			bMorphNeedsUpdate = false;
		}
	}

	const bool bGPUSkinCacheEnabled = GPUSkinCache && DynamicData->GPUSkinTechnique == ESkeletalMeshGPUSkinTechnique::GPUSkinCache && FGPUSkinCache::IsGPUSkinCacheEnabled(FeatureLevel);

	// Immediately release any stale entry if we've recently switched to a LOD level that disallows skin cache
	if (!bGPUSkinCacheEnabled)
	{
		bool bReleasedSkinCacheEntry = false;

		if (SkinCacheEntry)
		{
			FGPUSkinCache::Release(SkinCacheEntry);
			bReleasedSkinCacheEntry = true;
		}

#if RHI_RAYTRACING
		if (SkinCacheEntryForRayTracing)
		{
			FGPUSkinCache::Release(SkinCacheEntryForRayTracing);
			bReleasedSkinCacheEntry = true;
		}

		if (bReleasedSkinCacheEntry)
		{
			// When SkinCacheEntry is gone, clear geometry
			RayTracingGeometry.ReleaseRHI();
			RayTracingGeometry.SetInitializer(FRayTracingGeometryInitializer{});
		}
#endif // RHI_RAYTRACING
	}

	TArray<FGPUSkinCache::FProcessEntrySection, TInlineAllocator<8>> ProcessEntrySections;

	if (bGPUSkinCacheEnabled)
	{
		ProcessEntrySections.Reserve(Sections.Num());
	}

	if (LOD.UnifiedBoneBuffer)
	{
		LOD.UnifiedBoneBuffer->FlipBuffers();

		if (bHasPreviousReferenceToLocal)
		{
			const bool bPrevious = true;
			LOD.UnifiedBoneBuffer->Allocate(RHICmdList, bPrevious);
		}

		{
			const bool bPrevious = false;
			LOD.UnifiedBoneBuffer->Allocate(RHICmdList, bPrevious);
		}
	}

	for (int32 SectionIdx = 0; SectionIdx < Sections.Num(); SectionIdx++)
	{
		const FSkelMeshRenderSection& Section = Sections[SectionIdx];

		if (!Section.IsValid())
		{
			continue;
		}

		FGPUBaseSkinVertexFactory* VertexFactory = VertexFactoryData.VertexFactories[SectionIdx].Get();
		check(VertexFactory != nullptr);

		FGPUBaseSkinVertexFactory::FShaderDataType& ShaderData = VertexFactory->GetShaderData();
		ShaderData.SetRevisionNumbers(CurrentRevisionNumber, PreviousRevisionNumber);
		ShaderData.UpdatedFrameNumber = BoneTransformFrameNumber;

		if (BoneTransformAllocationMode == EBoneTransformAllocationMode::VertexFactory)
		{
			const auto AllocateBoneBuffer = [&](bool bPrevious)
			{
				FVertexBufferAndSRV& Buffer = ShaderData.GetBoneBufferForWriting(bPrevious);
				if (LOD.UnifiedBoneBuffer)
				{
					Buffer = LOD.UnifiedBoneBuffer->GetBuffer(bPrevious);
				}
				else
				{
					ShaderData.AllocateBoneBuffer(RHICmdList, VertexFactory->GetBoneBufferSize(), Buffer);
				}
			};

			if (bHasPreviousReferenceToLocal)
			{
				const bool bPrevious = true;
				AllocateBoneBuffer(bPrevious);
			}

			{
				const bool bPrevious = false;
				AllocateBoneBuffer(bPrevious);
			}
		}

		FGPUBaseSkinAPEXClothVertexFactory* ClothVertexFactory = VertexFactory->GetClothVertexFactory();
		FGPUBaseSkinAPEXClothVertexFactory::ClothShaderType* ClothShaderData = nullptr;

		const bool bSectionUsingCloth = GEnableCloth && ClothVertexFactory != nullptr;
		const bool bSectionUsingMorph = Mode == EGPUSkinCacheEntryMode::Raster && MorphVertexBuffer && !bSectionUsingCloth && (bHasExternalMorphs || (bHasWeightedActiveMorphs && DynamicData->SectionIdsUseByActiveMorphTargets.Contains(SectionIdx)));

		VertexFactory->UpdateMorphState(RHICmdList, bSectionUsingMorph);

		FMatrix44f ClothToLocal = FMatrix44f::Identity;
		const FClothSimulData* ClothSimulationData = nullptr;

		// Update uniform buffer for APEX cloth simulation mesh positions and normals
		if (bSectionUsingCloth)
		{
			ClothShaderData = &ClothVertexFactory->GetClothShaderData();
			ClothSimulationData = DynamicData->ClothingSimData.Find(Section.CorrespondClothAssetIndex);
			if (ClothSimulationData && ClothSimulationData->LODIndex > LODIndex)
			{
				// Can only render deform the simulation with positive LODBias which is
				// when the sim data LOD is lower than the requested render LODIndex.
				// Otherwise the cloth deformer mapping would be out of bounds.
				// This can happen when the physics is paused on a LOD >0 and then switching down the LODs.
				ClothSimulationData = nullptr;
			}

			ClothShaderData->bEnabled = ClothSimulationData != nullptr;

			if (ClothSimulationData)
			{
				ClothToLocal = FMatrix44f(ClothSimulationData->ComponentRelativeTransform.ToMatrixWithScale());

				if (!bGPUSkinCacheEnabled)
				{
					ClothShaderData->ClothBlendWeight = DynamicData->ClothBlendWeight;
					ClothShaderData->WorldScale = (FVector3f)WorldScale;
					ClothShaderData->UpdateClothSimulationData(RHICmdList, ClothSimulationData->Positions, ClothSimulationData->Normals, CurrentRevisionNumber, OwnerName);

					// Transform from cloth space to local space. Cloth space is relative to cloth root bone, local space is component space.
					ClothShaderData->GetClothToLocalForWriting() = ClothToLocal;
				}
			}
		}

		if (bGPUSkinCacheEnabled)
		{
			ProcessEntrySections.Emplace(FGPUSkinCache::FProcessEntrySection
			{
				  .SourceVertexFactory = VertexFactory
				, .Section             = &Section
				, .SectionIndex        = SectionIdx
				, .ClothSimulationData = ClothSimulationData
				, .ClothToLocal        = ClothToLocal
			});
		}

		if (Mode == EGPUSkinCacheEntryMode::Raster && DynamicData->GPUSkinTechnique != ESkeletalMeshGPUSkinTechnique::MeshDeformer && !bGPUSkinCacheEnabled)
		{
			if (BoneTransformAllocationMode == EBoneTransformAllocationMode::VertexFactory)
			{
				if (!VertexFactory->IsUniformBufferValid())
				{
					VertexFactory->UpdateUniformBuffer(RHICmdList, nullptr, nullptr);
				}
				else
				{
					VertexFactory->MarkUniformBufferDirty();
				}
			}

			// Mobile doesn't support motion blur so no need to double buffer cloth data.
			// Skin cache doesn't need double buffering, if failed to enter skin cache then the fall back GPU skinned VF needs double buffering.
			if (ClothSimulationData != nullptr && !bIsMobile)
			{
				ClothShaderData->EnableDoubleBuffer();
			}
		}
	}

	if (!ProcessEntrySections.IsEmpty())
	{
		FGPUSkinCacheEntry*& InOutEntry = Mode == EGPUSkinCacheEntryMode::RayTracing ? SkinCacheEntryForRayTracing : SkinCacheEntry;

		GPUSkinCache->ProcessEntry(RHICmdList, FGPUSkinCache::FProcessEntryInputs
		{
			  .Mode                  = Mode
			, .Sections              = ProcessEntrySections
			, .Skin                  = this
			, .TargetVertexFactory   = VertexFactoryData.PassthroughVertexFactory.Get()
			, .MorphVertexBuffer     = MorphVertexBuffer
			, .ClothVertexBuffer     = &LODData.ClothVertexBuffer
			, .ClothWorldScale       = (FVector3f)WorldScale
			, .ClothBlendWeight      = DynamicData->ClothBlendWeight
			, .CurrentRevisionNumber = CurrentRevisionNumber
			, .LODIndex              = LODIndex
			, .bRecreating           = DynamicData->bRecreating != 0

		}, InOutEntry);
	}

	if (Mode == EGPUSkinCacheEntryMode::Raster)
	{
		if (MorphVertexBuffer != nullptr && !LOD.MorphVertexBufferPool->IsDoubleBuffered() &&
			// Mobile doesn't support motion blur so no need to double buffer morph deltas.
			!bIsMobile &&
			// Skin cache / mesh deformers don't need double buffered morph targets.
			!bGPUSkinCacheEnabled && DynamicData->GPUSkinTechnique != ESkeletalMeshGPUSkinTechnique::MeshDeformer)
		{
			// At least one section is going through the base GPU skinned vertex factory so turn on double buffering for motion blur.
			LOD.MorphVertexBufferPool->EnableDoubleBuffer(RHICmdList);
		}

		bSupportsStaticRelevance = true;
	}
}

#if RHI_RAYTRACING

void FSkeletalMeshObjectGPUSkin::UpdateRayTracingGeometry_Internal(
	FSkeletalMeshLODRenderData& LODModel, uint32 LODIndex, TArray<FBufferRHIRef>& VertexBuffers,
	FRayTracingGeometry& RayTracingGeometry, bool bAnySegmentUsesWorldPositionOffset, FSkeletalMeshObject* MeshObject)
{
	if (IsRayTracingEnabled() && MeshObject->bSupportRayTracing)
	{
		// check(LODIndex == MeshObject->GetRayTracingLOD());
		bool bRequireRecreatingRayTracingGeometry = LODIndex != RayTracingGeometry.LODIndex
			|| MeshObject->bHiddenMaterialVisibilityDirtyForRayTracing
			|| RayTracingGeometry.GetInitializer().Segments.Num() == 0;

		if (!bRequireRecreatingRayTracingGeometry)
		{
			for (const FRayTracingGeometrySegment& Segment : RayTracingGeometry.GetInitializer().Segments)
			{
				if (Segment.VertexBuffer == nullptr)
				{
					bRequireRecreatingRayTracingGeometry = true;
					break;
				}
			}
		}
		MeshObject->bHiddenMaterialVisibilityDirtyForRayTracing = false;

		auto PatchSegmentVertexBuffer = [&LODModel](FRayTracingGeometrySegment& Segment)
			{
				if (Segment.bEnabled)
				{
					checkf(Segment.VertexBuffer != nullptr, TEXT("Must provide valid vertex buffer when segment is enabled."));
				}
				else if (Segment.VertexBuffer == nullptr) // TODO: Consider always doing this when !Segment.bEnabled
				{
					// TODO: RHIs should accept NULL VertexBuffer when Segment is disabled.
					// For now use the static PositionVertexBuffer as placeholder.
					// TODO: Consider also setting Segment.NumPrimitives = 0; in this case
					Segment.VertexBuffer = LODModel.StaticVertexBuffers.PositionVertexBuffer.GetRHI();
				}
			};

		if (bRequireRecreatingRayTracingGeometry)
		{
			FBufferRHIRef IndexBufferRHI = LODModel.MultiSizeIndexContainer.GetIndexBuffer()->IndexBufferRHI;
			uint32 VertexBufferNumVertices = LODModel.StaticVertexBuffers.PositionVertexBuffer.GetNumVertices();
			uint32 VertexBufferStride = LODModel.StaticVertexBuffers.PositionVertexBuffer.GetStride();

			FRayTracingGeometryInitializer Initializer;

#if !UE_BUILD_SHIPPING
			if (MeshObject->DebugName.IsValid())
			{
				Initializer.DebugName = MeshObject->DebugName;
			}
			else
#endif
			{
				static const FName DefaultDebugName("FSkeletalMeshObject");
				static int32 DebugNumber = 0;
				Initializer.DebugName = FName(DefaultDebugName, DebugNumber++);
			}

			Initializer.OwnerName = MeshObject->GetAssetPathName(LODIndex);
			Initializer.IndexBuffer = IndexBufferRHI;
			Initializer.GeometryType = RTGT_Triangles;
			Initializer.bFastBuild = true;
			Initializer.bAllowUpdate = true;

			Initializer.Segments.Reserve(LODModel.RenderSections.Num());

			for (int32 SectionIndex = 0; SectionIndex < LODModel.RenderSections.Num(); ++SectionIndex)
			{
				const FSkelMeshRenderSection& Section = LODModel.RenderSections[SectionIndex];

				FRayTracingGeometrySegment Segment;
				Segment.VertexBuffer = VertexBuffers[SectionIndex];
				Segment.VertexBufferElementType = VET_Float3;
				Segment.VertexBufferStride = VertexBufferStride;
				Segment.VertexBufferOffset = 0;
				Segment.MaxVertices = VertexBufferNumVertices;
				Segment.FirstPrimitive = Section.BaseIndex / 3;
				Segment.NumPrimitives = Section.NumTriangles;

				// TODO: If we are at a dropped LOD, route material index through the LODMaterialMap in the LODInfo struct.
				Segment.bEnabled = !MeshObject->IsMaterialHidden(LODIndex, Section.MaterialIndex) && Section.IsValid() && Section.bVisibleInRayTracing;

				PatchSegmentVertexBuffer(Segment);

				Initializer.Segments.Add(Segment);

				Initializer.TotalPrimitiveCount += Segment.NumPrimitives;
			}

			if (RayTracingGeometry.GetRHI() != nullptr)
			{
				// RayTracingGeometry.ReleaseRHI() releases the old RT geometry, however due to the deferred deletion nature of RHI resources
				// they will not be released until the end of the frame. We may get OOM in the middle of batched updates if not flushing.

				// Release the old data (make sure it's not pending build anymore either)
				RayTracingGeometry.GetRHI()->DisableLifetimeExtension();
				RayTracingGeometry.ReleaseRHI();
			}

			Initializer.SourceGeometry = LODModel.SourceRayTracingGeometry.GetRHI();

			RayTracingGeometry.LODIndex = LODIndex;

			// Update the new init data
			RayTracingGeometry.SetInitializer(MoveTemp(Initializer));
		}
		else if (!bAnySegmentUsesWorldPositionOffset)
		{
			check(LODModel.RenderSections.Num() == RayTracingGeometry.GetInitializer().Segments.Num());

			// Refit BLAS with new vertex buffer data
			for (int32 SectionIndex = 0; SectionIndex < LODModel.RenderSections.Num(); ++SectionIndex)
			{
				PRAGMA_DISABLE_DEPRECATION_WARNINGS
				// TODO: Shouldn't directly modify FRayTracingGeometry initializer
				FRayTracingGeometrySegment& Segment = RayTracingGeometry.Initializer.Segments[SectionIndex];
				PRAGMA_ENABLE_DEPRECATION_WARNINGS
				Segment.VertexBuffer = VertexBuffers[SectionIndex];
				Segment.VertexBufferOffset = 0;

				PatchSegmentVertexBuffer(Segment);
			}
		}

		// Geometry needs to be updated
		RayTracingGeometry.SetRequiresUpdate(true);
	}
}

void FSkeletalMeshObjectGPUSkin::UpdateRayTracingGeometry(FRHICommandListBase& RHICmdList, FSkeletalMeshLODRenderData& LODModel, uint32 LODIndex, TArray<FBufferRHIRef>& VertexBuffers)
{
	const bool bAnySegmentUsesWorldPositionOffset = DynamicData != nullptr ? DynamicData->bAnySegmentUsesWorldPositionOffset : false;

	UpdateRayTracingGeometry_Internal(LODModel, LODIndex, VertexBuffers, RayTracingGeometry, bAnySegmentUsesWorldPositionOffset, this);
}

#endif // RHI_RAYTRACING

bool FSkeletalMeshObjectGPUSkin::IsExternalMorphSetActive(int32 MorphSetID, const FExternalMorphSet& MorphSet) const
{
	const FMorphTargetVertexInfoBuffers& CompressedBuffers = MorphSet.MorphBuffers;
	FExternalMorphSetWeights* WeightData = DynamicData->ExternalMorphWeightData.MorphSets.Find(MorphSetID);
	return (WeightData &&
			WeightData->Weights.Num() == CompressedBuffers.GetNumMorphs() &&
			WeightData->NumActiveMorphTargets > 0);
}

static void CalculateMorphDeltaBoundsAccum(
	const TArray<float>& MorphTargetWeights,
	const FMorphTargetVertexInfoBuffers& MorphTargetVertexInfoBuffers,
	FVector4& MinAccumScale,
	FVector4& MaxAccumScale,
	FVector4& MaxScale)
{
	for (uint32 i = 0; i < MorphTargetVertexInfoBuffers.GetNumMorphs(); i++)
	{
		FVector4f MinMorphScale = MorphTargetVertexInfoBuffers.GetMinimumMorphScale(i);
		FVector4f MaxMorphScale = MorphTargetVertexInfoBuffers.GetMaximumMorphScale(i);

		for (uint32 j = 0; j < 4; j++)
		{
			if (MorphTargetWeights.IsValidIndex(i))
			{
				MinAccumScale[j] += MorphTargetWeights[i] * MinMorphScale[j];
				MaxAccumScale[j] += MorphTargetWeights[i] * MaxMorphScale[j];
			}

			double AbsMorphScale = FMath::Max<double>(FMath::Abs(MinMorphScale[j]), FMath::Abs(MaxMorphScale[j]));
			double AbsAccumScale = FMath::Max<double>(FMath::Abs(MinAccumScale[j]), FMath::Abs(MaxAccumScale[j]));

			// The maximum accumulated and the maximum local value have to fit into out int24.
			MaxScale[j] = FMath::Max(MaxScale[j], FMath::Max(AbsMorphScale, AbsAccumScale));
		}
	}
}

static void CalculateMorphDeltaBoundsIncludingExternalMorphs(
	const TArray<float>& MorphTargetWeights,
	const FMorphTargetVertexInfoBuffers& MorphTargetVertexInfoBuffers,
	const FExternalMorphSets& ExternalMorphSets,
	const TMap<int32, FExternalMorphSetWeights>& ExternalWeights,
	FVector4& MorphScale,
	FVector4& InvMorphScale)
{
	FVector4 MinAccumScale(0, 0, 0, 0);
	FVector4 MaxAccumScale(0, 0, 0, 0);
	FVector4 MaxScale(0, 0, 0, 0);

	// Include the standard morph targets.
	CalculateMorphDeltaBoundsAccum(MorphTargetWeights, MorphTargetVertexInfoBuffers, MinAccumScale, MaxAccumScale, MaxScale);

	// Include all external morph targets.
	for (const auto& MorphSet : ExternalMorphSets)
	{
		const int32 MorphSetID = MorphSet.Key;
		const FMorphTargetVertexInfoBuffers& CompressedBuffers = MorphSet.Value->MorphBuffers;
		const FExternalMorphSetWeights* WeightData = ExternalWeights.Find(MorphSetID);
		check(WeightData);
		CalculateMorphDeltaBoundsAccum(WeightData->Weights, CompressedBuffers, MinAccumScale, MaxAccumScale, MaxScale);
	}

	MaxScale[0] = FMath::Max<double>(MaxScale[0], 1.0);
	MaxScale[1] = FMath::Max<double>(MaxScale[1], 1.0);
	MaxScale[2] = FMath::Max<double>(MaxScale[2], 1.0);
	MaxScale[3] = FMath::Max<double>(MaxScale[3], 1.0);

	const double ScaleToInt24 = 16777216.0;

	MorphScale = FVector4
	(
		ScaleToInt24 / (MaxScale[0]),
		ScaleToInt24 / (MaxScale[1]),
		ScaleToInt24 / (MaxScale[2]),
		ScaleToInt24 / (MaxScale[3])
	);

	InvMorphScale = FVector4
	(
		MaxScale[0] / ScaleToInt24,
		MaxScale[1] / ScaleToInt24,
		MaxScale[2] / ScaleToInt24,
		MaxScale[3] / ScaleToInt24
	);
}

void FSkeletalMeshObjectGPUSkin::UpdateMorphVertexBuffer(FRHICommandList& RHICmdList)
{
	if (!MorphVertexBuffer)
	{
		return;
	}

	if (bMorphNeedsUpdate)
	{
		const EGPUSkinCacheEntryMode Mode = EGPUSkinCacheEntryMode::Raster;
		const int32 LODIndex = DynamicData->GetLODIndex(Mode);
		const FSkeletalMeshLODRenderData& LODData = SkeletalMeshRenderData->LODRenderData[LODIndex];
		FSkeletalMeshObjectLOD& LOD = LODs[LODIndex];

		if (UseGPUMorphTargets(FeatureLevel))
		{
			// Count all active external morph sets.
			int32 NumMorphSets = 1; // Start at one, as we have our standard morph targets as well.
			for (const auto& MorphSet : DynamicData->ExternalMorphSets)
			{
				if (IsExternalMorphSetActive(MorphSet.Key, *MorphSet.Value))
				{
					NumMorphSets++;
				}
			}

			int32 MorphSetIndex = 0;

			// Calculate the delta bounds.
			FVector4 MorphScale;
			FVector4 InvMorphScale;
			{
				SCOPE_CYCLE_COUNTER(STAT_MorphVertexBuffer_ApplyDelta);
				CalculateMorphDeltaBoundsIncludingExternalMorphs(
					DynamicData->MorphTargetWeights,
					LODData.MorphTargetVertexInfoBuffers,
					DynamicData->ExternalMorphSets,
					DynamicData->ExternalMorphWeightData.MorphSets,
					MorphScale,
					InvMorphScale);
			}

			// Sometimes this goes out of bound, we'll ensure here.
			ensureAlways(DynamicData->MorphTargetWeights.Num() == LODData.MorphTargetVertexInfoBuffers.GetNumMorphs());
			LOD.UpdateMorphVertexBufferGPU(
				RHICmdList, 
				DynamicData->MorphTargetWeights, 
				LODData.MorphTargetVertexInfoBuffers, 
				DynamicData->SectionIdsUseByActiveMorphTargets,
				GetDebugName(), 
				Mode, 
				*MorphVertexBuffer, 
				true,	// Only clear the morph vertex buffer at the first morph set.
				(MorphSetIndex == NumMorphSets-1),
				MorphScale,
				InvMorphScale);	// Normalize only after the last morph set.

			MorphSetIndex++;

			// Process all external morph targets.
			for (const auto& MorphSet : DynamicData->ExternalMorphSets)
			{
				const int32 MorphSetID = MorphSet.Key;
				const FMorphTargetVertexInfoBuffers& CompressedBuffers = MorphSet.Value->MorphBuffers;
				FExternalMorphSetWeights* WeightData = DynamicData->ExternalMorphWeightData.MorphSets.Find(MorphSetID);
				check(WeightData);
				if (IsExternalMorphSetActive(MorphSetID, *MorphSet.Value))
				{
					LOD.UpdateMorphVertexBufferGPU(
						RHICmdList, WeightData->Weights,
						CompressedBuffers,
						DynamicData->SectionIdsUseByActiveMorphTargets,
						GetDebugName(),
						Mode,
						*MorphVertexBuffer,
						false,	// Don't clear the vertex buffer as we already did with the standard morph targets above.
						(MorphSetIndex == NumMorphSets - 1),
						MorphScale,
						InvMorphScale);	// Normalize only after the last morph set.

					MorphSetIndex++;
				}
			}

			// If this hits, the CalcNumActiveGPUMorphSets most likely returns the wrong number.
			check(NumMorphSets == MorphSetIndex);
		}
		else
		{
			// update the morph data for the lod (before SkinCache)
			const bool bSkinCacheEnabled = IsSkinCacheEnabled(EGPUSkinCacheEntryMode::Raster);
			LOD.UpdateMorphVertexBufferCPU(RHICmdList, DynamicData->ActiveMorphTargets, DynamicData->MorphTargetWeights, DynamicData->SectionIdsUseByActiveMorphTargets, bSkinCacheEnabled, *MorphVertexBuffer);
		}
	}
	else if (MorphVertexBuffer->bNeedsInitialClear && MorphVertexBuffer->GetUAV())
	{
		SCOPED_DRAW_EVENTF(RHICmdList, MorphInitialClear, TEXT("MorphInitialClear"));
		RHICmdList.Transition(FRHITransitionInfo(MorphVertexBuffer->GetUAV(), ERHIAccess::Unknown, ERHIAccess::UAVCompute));
		RHICmdList.ClearUAVUint(MorphVertexBuffer->GetUAV(), FUintVector4(0, 0, 0, 0));
		RHICmdList.Transition(FRHITransitionInfo(MorphVertexBuffer->GetUAV(), ERHIAccess::UAVCompute, RHICmdList.IsAsyncCompute() ? ERHIAccess::SRVCompute : ERHIAccess::SRVMask));
	}

	MorphVertexBuffer->bNeedsInitialClear = false;
	MorphVertexBuffer->bHasBeenUpdated = true;
	bMorphNeedsUpdate = false;
}

TArray<float> FSkeletalMeshObjectGPUSkin::FSkeletalMeshObjectLOD::MorphAccumulatedWeightArray;

FGPUMorphUpdateCS::FGPUMorphUpdateCS() = default;

FGPUMorphUpdateCS::FGPUMorphUpdateCS(const ShaderMetaType::CompiledShaderInitializerType& Initializer)
	: FGlobalShader(Initializer)
{
	MorphVertexBufferParameter.Bind(Initializer.ParameterMap, TEXT("MorphVertexBuffer"));

	MorphTargetWeightsParameter.Bind(Initializer.ParameterMap, TEXT("MorphTargetWeights"));
	MorphTargetBatchOffsetsParameter.Bind(Initializer.ParameterMap, TEXT("MorphTargetBatchOffsets"));
	MorphTargetGroupOffsetsParameter.Bind(Initializer.ParameterMap, TEXT("MorphTargetGroupOffsets"));
	PositionScaleParameter.Bind(Initializer.ParameterMap, TEXT("PositionScale"));
	PrecisionParameter.Bind(Initializer.ParameterMap, TEXT("Precision"));
	NumGroupsParameter.Bind(Initializer.ParameterMap, TEXT("NumGroups"));

	MorphDataBufferParameter.Bind(Initializer.ParameterMap, TEXT("MorphDataBuffer"));
}

void FGPUMorphUpdateCS::SetParameters(
	FRHIBatchedShaderParameters& BatchedParameters,
	const FVector4& LocalScale,
	const FMorphTargetVertexInfoBuffers& MorphTargetVertexInfoBuffers,
	FMorphVertexBuffer& MorphVertexBuffer,
	uint32 NumGroups,
	uint32 BatchOffsets[MorphTargetDispatchBatchSize],
	uint32 GroupOffsets[MorphTargetDispatchBatchSize],
	float Weights[MorphTargetDispatchBatchSize])
{
	SetUAVParameter(BatchedParameters, MorphVertexBufferParameter, MorphVertexBuffer.GetUAV());

	SetShaderValue(BatchedParameters, PositionScaleParameter, (FVector4f)LocalScale);
	FVector2f Precision = { MorphTargetVertexInfoBuffers.GetPositionPrecision(), MorphTargetVertexInfoBuffers.GetTangentZPrecision() };
	SetShaderValue(BatchedParameters, PrecisionParameter, Precision);
	SetShaderValue(BatchedParameters, NumGroupsParameter, NumGroups);

	SetSRVParameter(BatchedParameters, MorphDataBufferParameter, MorphTargetVertexInfoBuffers.MorphDataSRV);

	SetShaderValue(BatchedParameters, MorphTargetBatchOffsetsParameter, *(uint32(*)[MorphTargetDispatchBatchSize]) BatchOffsets);
	SetShaderValue(BatchedParameters, MorphTargetGroupOffsetsParameter, *(uint32(*)[MorphTargetDispatchBatchSize]) GroupOffsets);
	SetShaderValue(BatchedParameters, MorphTargetWeightsParameter, *(float(*)[MorphTargetDispatchBatchSize]) Weights);
}

void FGPUMorphUpdateCS::Dispatch(FRHICommandList& RHICmdList, uint32 Size)
{
	const FIntVector DispatchSize = FComputeShaderUtils::GetGroupCountWrapped(Size);
	RHICmdList.DispatchComputeShader(DispatchSize.X, DispatchSize.Y, DispatchSize.Z);
}

void FGPUMorphUpdateCS::UnsetParameters(FRHIBatchedShaderUnbinds& BatchedUnbinds)
{
	UnsetUAVParameter(BatchedUnbinds, MorphVertexBufferParameter);
}

bool FGPUMorphUpdateCS::ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
{
	return IsFeatureLevelSupported(Parameters.Platform, ERHIFeatureLevel::SM5);
}

IMPLEMENT_SHADER_TYPE(, FGPUMorphUpdateCS, TEXT("/Engine/Private/MorphTargets.usf"), TEXT("GPUMorphUpdateCS"), SF_Compute);

FGPUMorphNormalizeCS::FGPUMorphNormalizeCS() = default;

FGPUMorphNormalizeCS::FGPUMorphNormalizeCS(const ShaderMetaType::CompiledShaderInitializerType& Initializer)
	: FGlobalShader(Initializer)
{
	MorphVertexBufferParameter.Bind(Initializer.ParameterMap, TEXT("MorphVertexBuffer"));
	PositionScaleParameter.Bind(Initializer.ParameterMap, TEXT("PositionScale"));
	NumVerticesParameter.Bind(Initializer.ParameterMap, TEXT("NumVertices"));
}

bool FGPUMorphNormalizeCS::ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
{
	return IsFeatureLevelSupported(Parameters.Platform, ERHIFeatureLevel::SM5);
}

void FGPUMorphNormalizeCS::SetParameters(FRHIBatchedShaderParameters& BatchedParameters, const FVector4& InvLocalScale, const FMorphTargetVertexInfoBuffers& MorphTargetVertexInfoBuffers, FMorphVertexBuffer& MorphVertexBuffer, uint32 NumVertices)
{
	SetUAVParameter(BatchedParameters, MorphVertexBufferParameter, MorphVertexBuffer.GetUAV());
	SetShaderValue(BatchedParameters, PositionScaleParameter, (FVector4f)InvLocalScale);
	SetShaderValue(BatchedParameters, NumVerticesParameter, NumVertices);
}

void FGPUMorphNormalizeCS::Dispatch(FRHICommandList& RHICmdList, uint32 NumVertices)
{
	FIntVector DispatchSize = FComputeShaderUtils::GetGroupCountWrapped(NumVertices, 64);
	RHICmdList.DispatchComputeShader(DispatchSize.X, DispatchSize.Y, DispatchSize.Z);
}

void FGPUMorphNormalizeCS::UnsetParameters(FRHIBatchedShaderUnbinds& BatchedUnbinds)
{
	UnsetUAVParameter(BatchedUnbinds, MorphVertexBufferParameter);
}

IMPLEMENT_SHADER_TYPE(, FGPUMorphNormalizeCS, TEXT("/Engine/Private/MorphTargets.usf"), TEXT("GPUMorphNormalizeCS"), SF_Compute);

FSkeletalMeshObjectGPUSkin::FSkeletalMeshObjectLOD::FSkeletalMeshObjectLOD(
	FSkeletalMeshRenderData* InSkelMeshRenderData,
	int32 InLOD,
	ERHIFeatureLevel::Type InFeatureLevel,
	FMorphVertexBufferPool* InRecreateBufferPool,
	ESkeletalMeshGPUSkinTechnique InSkinTechnique)
	: SkelMeshRenderData(InSkelMeshRenderData)
	, LODIndex(InLOD)
	, FeatureLevel(InFeatureLevel)
	, GPUSkinTechnique(InSkinTechnique)
{
	if (InRecreateBufferPool)
	{
		MorphVertexBufferPool = InRecreateBufferPool;
	}
	else
	{
		MorphVertexBufferPool = new FMorphVertexBufferPool(InSkelMeshRenderData, LODIndex, FeatureLevel);
	}
}

void FSkeletalMeshObjectGPUSkin::FSkeletalMeshObjectLOD::UpdateMorphVertexBufferGPU(
	FRHICommandList& RHICmdList, 
	const TArray<float>& MorphTargetWeights,
	const FMorphTargetVertexInfoBuffers& MorphTargetVertexInfoBuffers,
	const TArray<int32>& SectionIdsUseByActiveMorphTargets,
	const FName& OwnerName,
	EGPUSkinCacheEntryMode Mode,
	FMorphVertexBuffer& InMorphVertexBuffer,
	bool bClearMorphVertexBuffer,
	bool bNormalizePass,
	const FVector4& MorphScale,
	const FVector4& InvMorphScale)
{
	check(InMorphVertexBuffer.VertexBufferRHI);

	SCOPE_CYCLE_COUNTER(STAT_MorphVertexBuffer_Update);

	// LOD of the skel mesh is used to find number of vertices in buffer
	FSkeletalMeshLODRenderData& LodData = SkelMeshRenderData->LODRenderData[LODIndex];

	const bool bUseGPUMorphTargets = UseGPUMorphTargets(FeatureLevel);
	InMorphVertexBuffer.RecreateResourcesIfRequired(RHICmdList, bUseGPUMorphTargets);

	RHI_BREADCRUMB_EVENT_STAT_F(RHICmdList, MorphTargets, "MorphUpdate", "MorphUpdate%s_%s_LOD%d LodVertices=%d Batches=%d"
		, RHI_BREADCRUMB_FORCE_STRING_LITERAL(Mode == EGPUSkinCacheEntryMode::RayTracing ? TEXT("[RT]") : TEXT(""))
		, OwnerName
		, LODIndex
		, LodData.GetNumVertices()
		, MorphTargetVertexInfoBuffers.GetNumBatches()
	);

	RHICmdList.Transition(FRHITransitionInfo(InMorphVertexBuffer.GetUAV(), ERHIAccess::Unknown, ERHIAccess::UAVCompute));
	if (bClearMorphVertexBuffer)
	{
		RHICmdList.ClearUAVUint(InMorphVertexBuffer.GetUAV(), FUintVector4(0, 0, 0, 0));
	}

	if (MorphTargetVertexInfoBuffers.IsRHIInitialized() && (MorphTargetVertexInfoBuffers.GetNumMorphs() > 0))
	{
		{
			SCOPED_DRAW_EVENTF(RHICmdList, MorphUpdateScatter, TEXT("Scatter"));

			RHICmdList.Transition(FRHITransitionInfo(InMorphVertexBuffer.GetUAV(), ERHIAccess::UAVCompute, ERHIAccess::UAVCompute));
			RHICmdList.BeginUAVOverlap(InMorphVertexBuffer.GetUAV());

			//the first pass scatters all morph targets into the vertexbuffer using atomics
			//multiple morph targets can be batched by a single shader where the shader will rely on
			//binary search to find the correct target weight within the batch.
			TShaderMapRef<FGPUMorphUpdateCS> GPUMorphUpdateCS(GetGlobalShaderMap(FeatureLevel));

			uint32 InputMorphStartIndex = 0;
			while (InputMorphStartIndex < MorphTargetVertexInfoBuffers.GetNumMorphs())
			{
				uint32 BatchOffsets[FGPUMorphUpdateCS::MorphTargetDispatchBatchSize];
				uint32 GroupOffsets[FGPUMorphUpdateCS::MorphTargetDispatchBatchSize];
				float Weights[FGPUMorphUpdateCS::MorphTargetDispatchBatchSize];

				uint32 NumBatches = 0;
				uint32 NumOutputMorphs = 0;
				while (InputMorphStartIndex < MorphTargetVertexInfoBuffers.GetNumMorphs() && NumOutputMorphs < FGPUMorphUpdateCS::MorphTargetDispatchBatchSize)
				{
					if (MorphTargetWeights.IsValidIndex(InputMorphStartIndex) && MorphTargetWeights[InputMorphStartIndex] != 0.0f) 	// Omit morphs with zero weight
					{
						BatchOffsets[NumOutputMorphs] = MorphTargetVertexInfoBuffers.GetBatchStartOffset(InputMorphStartIndex);
						GroupOffsets[NumOutputMorphs] = NumBatches;
						Weights[NumOutputMorphs] = MorphTargetWeights[InputMorphStartIndex];
						NumOutputMorphs++;

						NumBatches += MorphTargetVertexInfoBuffers.GetNumBatches(InputMorphStartIndex);
					}
					InputMorphStartIndex++;
				}

				for (uint32 i = NumOutputMorphs; i < FGPUMorphUpdateCS::MorphTargetDispatchBatchSize; i++)
				{
					BatchOffsets[i] = 0;
					GroupOffsets[i] = NumBatches;
					Weights[i] = 0.0f;
				}

				SetComputePipelineState(RHICmdList, GPUMorphUpdateCS.GetComputeShader());

				SetShaderParametersLegacyCS(
					RHICmdList,
					GPUMorphUpdateCS,
					MorphScale,
					MorphTargetVertexInfoBuffers,
					InMorphVertexBuffer,
					NumBatches,
					BatchOffsets,
					GroupOffsets,
					Weights);

				GPUMorphUpdateCS->Dispatch(RHICmdList, NumBatches);
			}

			UnsetShaderParametersLegacyCS(RHICmdList, GPUMorphUpdateCS);
				
			RHICmdList.EndUAVOverlap(InMorphVertexBuffer.GetUAV());
		}

		if (bNormalizePass)
		{
			SCOPED_DRAW_EVENTF(RHICmdList, MorphUpdateNormalize, TEXT("Normalize"));

			RHICmdList.Transition(FRHITransitionInfo(InMorphVertexBuffer.GetUAV(), ERHIAccess::UAVCompute, ERHIAccess::UAVCompute));

			//The second pass normalizes the scattered result and converts it back into floats.
			//The dispatches are split by morph permutation (and their accumulated weight) .
			//Every vertex is touched only by a single permutation. 
			//multiple permutations can be batched by a single shader where the shader will rely on
			//binary search to find the correct target weight within the batch.
			TShaderMapRef<FGPUMorphNormalizeCS> GPUMorphNormalizeCS(GetGlobalShaderMap(FeatureLevel));

			SetComputePipelineState(RHICmdList, GPUMorphNormalizeCS.GetComputeShader());
			SetShaderParametersLegacyCS(RHICmdList, GPUMorphNormalizeCS, InvMorphScale, MorphTargetVertexInfoBuffers, InMorphVertexBuffer, InMorphVertexBuffer.GetNumVerticies());
			GPUMorphNormalizeCS->Dispatch(RHICmdList, InMorphVertexBuffer.GetNumVerticies());
			UnsetShaderParametersLegacyCS(RHICmdList, GPUMorphNormalizeCS);

			// When using async compute the skin cache is going to consume the contents of the buffer.
			RHICmdList.Transition(FRHITransitionInfo(InMorphVertexBuffer.GetUAV(), ERHIAccess::UAVCompute, RHICmdList.IsAsyncCompute() ? ERHIAccess::SRVCompute : ERHIAccess::VertexOrIndexBuffer | ERHIAccess::SRVMask));
		}
	}
}

FSkinWeightVertexBufferInfo GetSkinWeightVertexBufferInfo(const FSkinWeightVertexBuffer* Buffer)
{
	FSkinWeightVertexBufferInfo Info;

	if (Buffer)
	{
		Info.InputWeightStreamSRV		= Buffer->GetDataVertexBuffer()->GetSRV();
		Info.InputWeightLookupStreamSRV = Buffer->GetLookupVertexBuffer()->GetSRV();
		Info.InputWeightIndexSize       = Buffer->GetBoneIndexByteSize() | (Buffer->GetBoneWeightByteSize() << 8);
		Info.InputWeightStride          = Buffer->GetConstantInfluencesVertexStride();
		Info.MaxBoneInfluences          = Buffer->GetMaxBoneInfluences();
		Info.BoneInfluenceType          = Buffer->GetBoneInfluenceType();
		Info.bUse16BitBoneIndex         = Buffer->Use16BitBoneIndex();
		Info.bUse16BitBoneWeight        = Buffer->Use16BitBoneWeight();
	}

	return Info;
}

void FSkeletalMeshObjectGPUSkin::FSkeletalMeshObjectLOD::UpdateSkinWeights(const FSkelMeshComponentLODInfo* CompLODInfo, EBoneTransformStorageMode InBoneTransformStorageMode)
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_FSkeletalMeshObjectLOD_UpdateSkinWeights);

	check(SkelMeshRenderData);
	check(SkelMeshRenderData->LODRenderData.IsValidIndex(LODIndex));

	// If we have a skin weight override buffer (and it's the right size) use it
	FSkeletalMeshLODRenderData& LODData = SkelMeshRenderData->LODRenderData[LODIndex];	
	if (CompLODInfo)
	{
		const FSkinWeightVertexBuffer* NewMeshObjectWeightBuffer = FSkeletalMeshObject::GetSkinWeightVertexBuffer(LODData, CompLODInfo);
		if (MeshObjectWeightBuffer != NewMeshObjectWeightBuffer)
		{
			MeshObjectWeightBuffer = NewMeshObjectWeightBuffer;

			FVertexFactoryBuffers VertexBuffers;
			GetVertexBuffers(VertexBuffers, LODData, InBoneTransformStorageMode);

			FSkeletalMeshObjectLOD* Self = this;

			ENQUEUE_RENDER_COMMAND(UpdateSkinWeightsGPUSkin)(UE::RenderCommandPipe::SkeletalMesh,
				[NewMeshObjectWeightBuffer, VertexBuffers, Self](FRHICommandList& RHICmdList) mutable
			{
				Self->MeshObjectWeightBuffer_RenderThread = GetSkinWeightVertexBufferInfo(VertexBuffers.SkinWeightVertexBuffer);

				Self->GPUSkinVertexFactories.UpdateVertexFactoryData(VertexBuffers);
			});
		}
	}
}

void FSkeletalMeshObjectGPUSkin::FSkeletalMeshObjectLOD::UpdateMorphVertexBufferCPU(
	FRHICommandList& RHICmdList,
	const FMorphTargetWeightMap& InActiveMorphTargets,
	const TArray<float>& MorphTargetWeights,
	const TArray<int32>& SectionIdsUseByActiveMorphTargets,
	bool bGPUSkinCacheEnabled,
	FMorphVertexBuffer& InMorphVertexBuffer)
{
	SCOPE_CYCLE_COUNTER(STAT_MorphVertexBuffer_Update);
	check(IsValidRef(InMorphVertexBuffer.VertexBufferRHI));

	// LOD of the skel mesh is used to find number of vertices in buffer
	FSkeletalMeshLODRenderData& LodData = SkelMeshRenderData->LODRenderData[LODIndex];

	// Whether all sections of the LOD perform GPU recompute tangent
	const int32 RecomputeTangentsMode = FGPUSkinCache::GetGPUSkinCacheRecomputeTangentsMode(FeatureLevel);
	bool bAllSectionsDoGPURecomputeTangent = bGPUSkinCacheEnabled && RecomputeTangentsMode > 0;
	if (bAllSectionsDoGPURecomputeTangent && RecomputeTangentsMode == 2)
	{
		for (int32 i = 0; i < LodData.RenderSections.Num(); ++i)
		{
			const FSkelMeshRenderSection& RenderSection = LodData.RenderSections[i];
			if (RenderSection.NumTriangles > 0 && !RenderSection.bRecomputeTangent)
			{
				bAllSectionsDoGPURecomputeTangent = false;
				break;
			}
		}
	}

	// If the LOD performs GPU skin cache recompute tangent, then there is no need to update tangents here
	bool bBlendTangentsOnCPU = !bAllSectionsDoGPURecomputeTangent;

	const bool bUseGPUMorphTargets = UseGPUMorphTargets(FeatureLevel);
	InMorphVertexBuffer.RecreateResourcesIfRequired(RHICmdList, bUseGPUMorphTargets);

	uint32 Size = LodData.GetNumVertices() * sizeof(FMorphGPUSkinVertex);

	FMorphGPUSkinVertex* Buffer = nullptr;
	{
		SCOPE_CYCLE_COUNTER(STAT_MorphVertexBuffer_Alloc);
		Buffer = (FMorphGPUSkinVertex*)FMemory::Malloc(Size);
	}

	{
		SCOPE_CYCLE_COUNTER(STAT_MorphVertexBuffer_Init);

		if (bBlendTangentsOnCPU)
		{
			// zero everything
			int32 vertsToAdd = static_cast<int32>(LodData.GetNumVertices()) - MorphAccumulatedWeightArray.Num();
			if (vertsToAdd > 0)
			{
				MorphAccumulatedWeightArray.AddUninitialized(vertsToAdd);
			}

			FMemory::Memzero(MorphAccumulatedWeightArray.GetData(), sizeof(float)*LodData.GetNumVertices());
		}

		// PackedNormals will be wrong init with 0, but they'll be overwritten later
		FMemory::Memzero(&Buffer[0], sizeof(FMorphGPUSkinVertex)*LodData.GetNumVertices());
	}

	{
		SCOPE_CYCLE_COUNTER(STAT_MorphVertexBuffer_ApplyDelta);

		const float MorphTargetMaxBlendWeight = UE::SkeletalRender::Settings::GetMorphTargetMaxBlendWeight();

		// iterate over all active morph targets and accumulate their vertex deltas
		for(const TTuple<const UMorphTarget*, int32>& MorphItem: InActiveMorphTargets)
		{
			const UMorphTarget* MorphTarget = MorphItem.Key;
			const int32 WeightIndex = MorphItem.Value;
			checkSlow(MorphTarget != nullptr);
			checkSlow(MorphTarget->HasDataForLOD(LODIndex));
			const float MorphTargetWeight = MorphTargetWeights.IsValidIndex(WeightIndex) ? MorphTargetWeights[WeightIndex] : 0.0f;
			const float MorphAbsWeight = FMath::Abs(MorphTargetWeight);
			checkSlow(MorphAbsWeight >= MinMorphTargetBlendWeight && MorphAbsWeight <= MorphTargetMaxBlendWeight);

			// iterate over the vertices that this lod model has changed
			for (FMorphTargetDeltaIterator DeltaIt = MorphTarget->GetDeltaIteratorForLOD(LODIndex); !DeltaIt.AtEnd(); ++DeltaIt)
			{
				if (ensure(DeltaIt->SourceIdx < LodData.GetNumVertices()))
				{
					FMorphGPUSkinVertex& DestVertex = Buffer[DeltaIt->SourceIdx];
					DestVertex.DeltaPosition += DeltaIt->PositionDelta * MorphTargetWeight;

					// todo: could be moved out of the inner loop to be more efficient
					if (bBlendTangentsOnCPU)
					{
						DestVertex.DeltaTangentZ += DeltaIt->TangentZDelta * MorphTargetWeight;
						// accumulate the weight so we can normalized it later
						MorphAccumulatedWeightArray[DeltaIt->SourceIdx] += MorphAbsWeight;
					}
				}
			} // for all vertices
		} // for all morph targets

		if (bBlendTangentsOnCPU)
		{
			// copy back all the tangent values (can't use Memcpy, since we have to pack the normals)
			for (uint32 iVertex = 0; iVertex < LodData.GetNumVertices(); ++iVertex)
			{
				FMorphGPUSkinVertex& DestVertex = Buffer[iVertex];
				float AccumulatedWeight = MorphAccumulatedWeightArray[iVertex];

				// if accumulated weight is >1.f
				// previous code was applying the weight again in GPU if less than 1, but it doesn't make sense to do so
				// so instead, we just divide by AccumulatedWeight if it's more than 1.
				// now DeltaTangentZ isn't FPackedNormal, so you can apply any value to it. 
				if (AccumulatedWeight > 1.f)
				{
					DestVertex.DeltaTangentZ /= AccumulatedWeight;
				}
			}
		}
	} // ApplyDelta

	{
		SCOPE_CYCLE_COUNTER(STAT_MorphVertexBuffer_RhiLockAndCopy);
		FMorphGPUSkinVertex* ActualBuffer = (FMorphGPUSkinVertex*)RHICmdList.LockBuffer(InMorphVertexBuffer.VertexBufferRHI, 0, Size, RLM_WriteOnly);
		FMemory::Memcpy(ActualBuffer, Buffer, Size);
		RHICmdList.UnlockBuffer(InMorphVertexBuffer.VertexBufferRHI);
		FMemory::Free(Buffer);
	}
}

const FVertexFactory* FSkeletalMeshObjectGPUSkin::GetSkinVertexFactory(const FSceneView* View, int32 LODIndex, int32 ChunkIdx, ESkinVertexFactoryMode VFMode) const
{
	checkSlow( LODs.IsValidIndex(LODIndex) );
	checkSlow( DynamicData );

	const FSkeletalMeshObjectLOD& LOD = LODs[LODIndex];

	// If a mesh deformer cache was used, return the passthrough vertex factory
	if (DynamicData->GPUSkinTechnique == ESkeletalMeshGPUSkinTechnique::MeshDeformer)
	{
		if (LOD.GPUSkinVertexFactories.PassthroughVertexFactory)
		{
			return LOD.GPUSkinVertexFactories.PassthroughVertexFactory.Get();
		}
		return nullptr;
	}

#if RHI_RAYTRACING
	// Return the passthrough vertex factory if it is requested (by ray tracing)
	if (VFMode == ESkinVertexFactoryMode::RayTracing)
	{
		check(GetSkinCacheEntryForRayTracing());
		check(FGPUSkinCache::IsEntryValid(GetSkinCacheEntryForRayTracing(), ChunkIdx));

		return LOD.GPUSkinVertexFactories.PassthroughVertexFactory.Get();
	}
#endif

	// Return the passthrough vertes factory for Skin Cache.
	if (DynamicData->GPUSkinTechnique == ESkeletalMeshGPUSkinTechnique::GPUSkinCache)
	{
		check(FGPUSkinCache::IsEntryValid(SkinCacheEntry, ChunkIdx));
		
		return LOD.GPUSkinVertexFactories.PassthroughVertexFactory.Get();
	}

	// Shouldn't get here if inline shaders were disabled.
	check(AreInlineSkinningShadersEnabled(GetFeatureLevelShaderPlatform(FeatureLevel)))

	// No passthrough usage so return the base skin vertex factory.
	return GetBaseSkinVertexFactory(LODIndex, ChunkIdx);
}

const FVertexFactory* FSkeletalMeshObjectGPUSkin::GetStaticSkinVertexFactory(int32 LODIndex, int32 ChunkIdx, ESkinVertexFactoryMode VFMode) const
{
	const FSkeletalMeshObjectLOD& LOD = LODs[LODIndex];

	if (LOD.GPUSkinTechnique != ESkeletalMeshGPUSkinTechnique::Inline || VFMode == ESkinVertexFactoryMode::RayTracing)
	{
		if (LOD.GPUSkinVertexFactories.PassthroughVertexFactory)
		{
			return LOD.GPUSkinVertexFactories.PassthroughVertexFactory.Get();
		}
		return nullptr;
	}

	const FGPUBaseSkinVertexFactory* VertexFactory = LOD.GPUSkinVertexFactories.VertexFactories[ChunkIdx].Get();
	check(VertexFactory == nullptr || VertexFactory->IsReadyForStaticMeshCaching());
	return VertexFactory;
}

FGPUBaseSkinVertexFactory const* FSkeletalMeshObjectGPUSkin::GetBaseSkinVertexFactory(int32 LODIndex, int32 ChunkIdx) const
{
	return LODs[LODIndex].GPUSkinVertexFactories.VertexFactories[ChunkIdx].Get();
}

FSkinWeightVertexBufferInfo FSkeletalMeshObjectGPUSkin::GetSkinWeightVertexBuffer_RenderThread(int32 LODIndex) const
{
	check(LODs.IsValidIndex(LODIndex));

	return LODs[LODIndex].MeshObjectWeightBuffer_RenderThread;
}

FSkinningSceneExtensionProxy* FSkeletalMeshObjectGPUSkin::CreateSceneExtensionProxy(const USkinnedAsset* InSkinnedAsset, bool bAllowScaling) const
{
	if (IsGPUSkinSceneExtensionEnabled() && BoneTransformAllocationMode == EBoneTransformAllocationMode::Scene)
	{
		return new FSkinningSceneExtensionProxy(this, InSkinnedAsset, bAllowScaling);
	}
	return nullptr;
}

void FSkeletalMeshObjectGPUSkin::UpdateSceneExtensionHeader(FRHICommandListBase& RHICmdList, const FSkinningHeader& SkinningHeader, FRHIShaderResourceView* TransformDataBufferSRV) const
{
	check(DynamicData);
	const FSkeletalMeshObjectLOD& LOD = LODs[DynamicData->LODIndex];
	check(BoneTransformAllocationMode == EBoneTransformAllocationMode::Scene);

	// Cache for GetSectionBoneBuffer.
	SceneExtensionBoneBuffer.TransformBufferSRV = TransformDataBufferSRV;
	SceneExtensionBoneBuffer.TransformBufferOffset = SkinningHeader.TransformBufferOffset;
	SceneExtensionBoneBuffer.MaxTransformCount = SkinningHeader.MaxTransformCount;
	SceneExtensionBoneBuffer.CurrentTransformSlot = SkinningHeader.CurrentTransformSlot;

	for (const TUniquePtr<FGPUBaseSkinVertexFactory>& VertexFactory : LOD.GPUSkinVertexFactories.VertexFactories)
	{
		if (VertexFactory)
		{
			VertexFactory->UpdateUniformBuffer(RHICmdList, &SkinningHeader, TransformDataBufferSRV);
		}
	}
}

void FSkeletalMeshObjectGPUSkin::RefreshClothingTransforms(const FMatrix& InNewLocalToWorld, uint32 FrameNumber)
{
	if(DynamicData && DynamicData->ClothingSimData.Num() > 0)
	{
		FSkeletalMeshObjectLOD& LOD = LODs[DynamicData->LODIndex];
		const TArray<FSkelMeshRenderSection>& Sections = GetRenderSections(DynamicData->LODIndex);
		const int32 NumSections = Sections.Num();

		DynamicData->ClothObjectLocalToWorld = InNewLocalToWorld;

		for(int32 SectionIndex = 0; SectionIndex < NumSections; ++SectionIndex)
		{
			if (LOD.GPUSkinVertexFactories.VertexFactories.IsValidIndex(SectionIndex) &&
				LOD.GPUSkinVertexFactories.VertexFactories[SectionIndex].IsValid())
			{
				FGPUBaseSkinAPEXClothVertexFactory* ClothFactory = LOD.GPUSkinVertexFactories.VertexFactories[SectionIndex]->GetClothVertexFactory();

				if(ClothFactory)
				{
					const FSkelMeshRenderSection& Section = Sections[SectionIndex];
					FGPUBaseSkinAPEXClothVertexFactory::ClothShaderType& ClothShaderData = ClothFactory->GetClothShaderData();
					const int16 ActorIdx = Section.CorrespondClothAssetIndex;

					if(FClothSimulData* SimData = DynamicData->ClothingSimData.Find(ActorIdx))
					{
						ClothShaderData.GetClothToLocalForWriting() = FMatrix44f(SimData->ComponentRelativeTransform.ToMatrixWithScale());
					}
				}
			}
		}
	}
}

/** 
 * Initialize the stream components common to all GPU skin vertex factory types 
 *
 * @param VertexFactoryData - context for setting the vertex factory stream components. commited later
 * @param VertexBuffers - vertex buffers which contains the data and also stride info
 * @param bUseInstancedVertexWeights - use instanced influence weights instead of default weights
 */
void InitGPUSkinVertexFactoryComponents(FGPUSkinDataType* VertexFactoryData, const FSkeletalMeshObjectGPUSkin::FVertexFactoryBuffers& VertexBuffers, FGPUBaseSkinVertexFactory* VertexFactory)
{
	//position
	VertexBuffers.StaticVertexBuffers->PositionVertexBuffer.BindPositionVertexBuffer(VertexFactory, *VertexFactoryData);

	// tangents
	VertexBuffers.StaticVertexBuffers->StaticMeshVertexBuffer.BindTangentVertexBuffer(VertexFactory, *VertexFactoryData);
	VertexBuffers.StaticVertexBuffers->StaticMeshVertexBuffer.BindPackedTexCoordVertexBuffer(VertexFactory, *VertexFactoryData, MAX_TEXCOORDS);

	const FSkinWeightVertexBuffer* WeightBuffer = VertexBuffers.SkinWeightVertexBuffer; 
	const bool bUse16BitBoneIndex = WeightBuffer->Use16BitBoneIndex();
	const bool bUse16BitBoneWeight = WeightBuffer->Use16BitBoneWeight();
	VertexFactoryData->bUse16BitBoneIndex = bUse16BitBoneIndex;
	VertexFactoryData->NumBoneInfluences = WeightBuffer->GetMaxBoneInfluences();

	FGPUBaseSkinVertexFactory::FShaderDataType* ShaderData = VertexFactory ? &VertexFactory->GetShaderData() : nullptr;

	GPUSkinBoneInfluenceType BoneInfluenceType = WeightBuffer->GetBoneInfluenceType();
	if (BoneInfluenceType == GPUSkinBoneInfluenceType::UnlimitedBoneInfluence)
	{
		if (ShaderData)
		{
			ShaderData->InputWeightIndexSize = VertexBuffers.SkinWeightVertexBuffer->GetBoneIndexByteSize() | (VertexBuffers.SkinWeightVertexBuffer->GetBoneWeightByteSize() << 8);
			ShaderData->InputWeightStream = VertexBuffers.SkinWeightVertexBuffer->GetDataVertexBuffer()->GetSRV();
		}

		const FSkinWeightLookupVertexBuffer* LookupVertexBuffer = WeightBuffer->GetLookupVertexBuffer();
		VertexFactoryData->BlendOffsetCount = FVertexStreamComponent(LookupVertexBuffer, 0, LookupVertexBuffer->GetStride(), VET_UInt);
	}
	else
	{
		// bone indices & weights
		const FSkinWeightDataVertexBuffer* WeightDataVertexBuffer = WeightBuffer->GetDataVertexBuffer();
		const uint32 Stride = WeightBuffer->GetConstantInfluencesVertexStride();
		const uint32 WeightsOffset = WeightBuffer->GetConstantInfluencesBoneWeightsOffset();
		VertexFactoryData->BoneIndices = FVertexStreamComponent(WeightDataVertexBuffer, 0, Stride, bUse16BitBoneIndex ? VET_UShort4 : VET_UByte4);
		VertexFactoryData->BoneWeights = FVertexStreamComponent(WeightDataVertexBuffer, WeightsOffset, Stride, bUse16BitBoneWeight ? VET_UShort4N : VET_UByte4N);

		if (VertexFactoryData->NumBoneInfluences > MAX_INFLUENCES_PER_STREAM)
		{
			// Extra streams for bone indices & weights
			VertexFactoryData->ExtraBoneIndices = FVertexStreamComponent(
				WeightDataVertexBuffer, 4 * VertexBuffers.SkinWeightVertexBuffer->GetBoneIndexByteSize(), Stride, bUse16BitBoneIndex ? VET_UShort4 : VET_UByte4);
			VertexFactoryData->ExtraBoneWeights = FVertexStreamComponent(
				WeightDataVertexBuffer, WeightsOffset + 4 * VertexBuffers.SkinWeightVertexBuffer->GetBoneWeightByteSize(), Stride, bUse16BitBoneWeight ? VET_UShort4N : VET_UByte4N);
		}
	}

	// Color data may be NULL
	if( VertexBuffers.ColorVertexBuffer != NULL && 
		VertexBuffers.ColorVertexBuffer->IsInitialized() )
	{
		// Color
		VertexBuffers.ColorVertexBuffer->BindColorVertexBuffer(VertexFactory, *VertexFactoryData);
	}
	else
	{
		VertexFactoryData->ColorComponentsSRV = nullptr;
		VertexFactoryData->ColorIndexMask = 0;
	}

	VertexFactoryData->bMorphTarget = false;
	VertexFactoryData->MorphVertexBufferPool = VertexBuffers.MorphVertexBufferPool;

	// delta positions for morph targets
	VertexFactoryData->DeltaPositionComponent = FVertexStreamComponent(
		nullptr, STRUCT_OFFSET(FMorphGPUSkinVertex, DeltaPosition), sizeof(FMorphGPUSkinVertex), VET_Float3, EVertexStreamUsage::Overridden);

	// delta normals for morph targets
	VertexFactoryData->DeltaTangentZComponent = FVertexStreamComponent(
		nullptr, STRUCT_OFFSET(FMorphGPUSkinVertex, DeltaTangentZ), sizeof(FMorphGPUSkinVertex), VET_Float3, EVertexStreamUsage::Overridden);
}

/** 
 * Initialize the stream components common to all GPU skin vertex factory types 
 *
 * @param VertexFactoryData - context for setting the vertex factory stream components. commited later
 * @param VertexBuffers - vertex buffers which contains the data and also stride info
 * @param bUseInstancedVertexWeights - use instanced influence weights instead of default weights
 */
void InitAPEXClothVertexFactoryComponents(FGPUSkinAPEXClothDataType* VertexFactoryData,
										const FSkeletalMeshObjectGPUSkin::FVertexFactoryBuffers& VertexBuffers)
{
	VertexFactoryData->ClothBuffer = VertexBuffers.APEXClothVertexBuffer->GetSRV();
	VertexFactoryData->ClothIndexMapping = VertexBuffers.APEXClothVertexBuffer->GetClothIndexMapping();
}

/** 
 * Handles transferring data between game/render threads when initializing vertex factory components 
 */
class FDynamicUpdateVertexFactoryData
{
public:
	FDynamicUpdateVertexFactoryData(
		FGPUBaseSkinVertexFactory* InVertexFactory,
		const FSkeletalMeshObjectGPUSkin::FVertexFactoryBuffers& InVertexBuffers)
		:	VertexFactory(InVertexFactory)
		,	VertexBuffers(InVertexBuffers)
	{}

	FGPUBaseSkinVertexFactory* VertexFactory;
	const FSkeletalMeshObjectGPUSkin::FVertexFactoryBuffers VertexBuffers;
};

static FPSOPrecacheVertexFactoryData GetVertexFactoryData(FSkeletalMeshLODRenderData& LODRenderData, FGPUSkinDataType& GPUSkinDataType, ERHIFeatureLevel::Type FeatureLevel)
{
	const FVertexFactoryType* VertexFactoryType;
	FVertexDeclarationElementList VertexElements;
	if (LODRenderData.SkinWeightVertexBuffer.GetBoneInfluenceType() == GPUSkinBoneInfluenceType::DefaultBoneInfluence)
	{
		VertexFactoryType = &TGPUSkinVertexFactory<GPUSkinBoneInfluenceType::DefaultBoneInfluence>::StaticType;
		TGPUSkinVertexFactory<GPUSkinBoneInfluenceType::DefaultBoneInfluence>::GetVertexElements(FeatureLevel, EVertexInputStreamType::Default, GPUSkinDataType, VertexElements);
	}
	else
	{
		VertexFactoryType = &TGPUSkinVertexFactory<GPUSkinBoneInfluenceType::UnlimitedBoneInfluence>::StaticType;
		TGPUSkinVertexFactory<GPUSkinBoneInfluenceType::UnlimitedBoneInfluence>::GetVertexElements(FeatureLevel, EVertexInputStreamType::Default, GPUSkinDataType, VertexElements);
	}
	return FPSOPrecacheVertexFactoryData(VertexFactoryType, VertexElements);
}

static void InitPassthroughVertexFactory_RenderThread(
	FRHICommandList& RHICmdList,
	TUniquePtr<FGPUSkinPassthroughVertexFactory>* PassthroughVertexFactory,
	FGPUBaseSkinVertexFactory* SourceVertexFactory,
	ERHIFeatureLevel::Type FeatureLevel,
	FGPUSkinPassthroughVertexFactory::EVertexAttributeFlags VertexAttributeMask)
{
	if (PassthroughVertexFactory && !*PassthroughVertexFactory)
	{
		FLocalVertexFactory::FDataType Data;
		SourceVertexFactory->CopyDataTypeForLocalVertexFactory(Data);
		*PassthroughVertexFactory = MakeUnique<FGPUSkinPassthroughVertexFactory>(FeatureLevel, VertexAttributeMask, SourceVertexFactory->GetNumVertices());
		(*PassthroughVertexFactory)->SetData(RHICmdList, Data);
		(*PassthroughVertexFactory)->InitResource(RHICmdList);
	}
}

/**
 * Creates a vertex factory entry for the given type and initialize it on the render thread
 */
void FSkeletalMeshObjectGPUSkin::CreateVertexFactory(
	FRHICommandList& RHICmdList,
	TArray<TUniquePtr<FGPUBaseSkinVertexFactory>>& VertexFactories,
	TUniquePtr<FGPUSkinPassthroughVertexFactory>* PassthroughVertexFactory,
	const FSkeletalMeshObjectGPUSkin::FVertexFactoryBuffers& VertexBuffers,
	ERHIFeatureLevel::Type FeatureLevel,
	FGPUSkinPassthroughVertexFactory::EVertexAttributeFlags VertexAttributeMask,
	uint32 NumBones,
	uint32 BoneOffset,
	uint32 BaseVertexIndex,
	bool bUsedForPassthroughVertexFactory,
	bool bUnifiedBoneBuffer)
{
	FGPUBaseSkinVertexFactory* VertexFactory = nullptr;
	GPUSkinBoneInfluenceType BoneInfluenceType = VertexBuffers.SkinWeightVertexBuffer->GetBoneInfluenceType();

	const FGPUBaseSkinVertexFactory::FInitializer Initializer
	{
		  .FeatureLevel    = FeatureLevel
		, .NumBones        = NumBones
		, .BoneOffset      = BoneOffset
		, .NumVertices     = VertexBuffers.NumVertices
		, .BaseVertexIndex = BaseVertexIndex
		, .bUsedForPassthroughVertexFactory = bUsedForPassthroughVertexFactory
		, .bUnifiedBoneBuffer = bUnifiedBoneBuffer
	};

	if (BoneInfluenceType == GPUSkinBoneInfluenceType::DefaultBoneInfluence)
	{
		VertexFactory = new TGPUSkinVertexFactory<GPUSkinBoneInfluenceType::DefaultBoneInfluence>(Initializer);
	}
	else
	{
		VertexFactory = new TGPUSkinVertexFactory<GPUSkinBoneInfluenceType::UnlimitedBoneInfluence>(Initializer);
	}
	VertexFactories.Add(TUniquePtr<FGPUBaseSkinVertexFactory>(VertexFactory));

	FDynamicUpdateVertexFactoryData VertexUpdateData(VertexFactory, VertexBuffers);

	FGPUSkinDataType Data;
	InitGPUSkinVertexFactoryComponents(&Data, VertexUpdateData.VertexBuffers, VertexUpdateData.VertexFactory);
	VertexUpdateData.VertexFactory->SetData(RHICmdList, &Data);
	VertexUpdateData.VertexFactory->InitResource(RHICmdList);

	InitPassthroughVertexFactory_RenderThread(RHICmdList, PassthroughVertexFactory, VertexUpdateData.VertexFactory, FeatureLevel, VertexAttributeMask);
}

void UpdateVertexFactory(
	TArray<TUniquePtr<FGPUBaseSkinVertexFactory>>& VertexFactories,
	const FSkeletalMeshObjectGPUSkin::FVertexFactoryBuffers& InVertexBuffers)
{
	for (TUniquePtr<FGPUBaseSkinVertexFactory>& FactoryPtr : VertexFactories)
	{
		FGPUBaseSkinVertexFactory* VertexFactory = FactoryPtr.Get();

		if (VertexFactory != nullptr)
		{
			// Setup the update data for enqueue
			FDynamicUpdateVertexFactoryData VertexUpdateData(VertexFactory, InVertexBuffers);

			// update vertex factory components and sync it
			ENQUEUE_RENDER_COMMAND(UpdateGPUSkinVertexFactory)(UE::RenderCommandPipe::SkeletalMesh,
				[VertexUpdateData](FRHICommandList& RHICmdList)
			{
				// Do not recreate the factory if it's been released, given the loose scheduling this may result in dangling factories
				if (!VertexUpdateData.VertexFactory->IsInitialized())
				{
					return;
				}

				// Use the cloth data type for both variants since the base version will just ignore the cloth parts.
				FGPUSkinAPEXClothDataType Data;
				InitGPUSkinVertexFactoryComponents(&Data, VertexUpdateData.VertexBuffers, VertexUpdateData.VertexFactory);
				InitAPEXClothVertexFactoryComponents(&Data, VertexUpdateData.VertexBuffers);
				VertexUpdateData.VertexFactory->SetData(RHICmdList, &Data);
				VertexUpdateData.VertexFactory->InitResource(RHICmdList);
			});
		}
	}
}

// APEX cloth

static FPSOPrecacheVertexFactoryData GetVertexFactoryDataCloth(FSkeletalMeshLODRenderData& LODRenderData, FGPUSkinDataType& GPUSkinDataType, ERHIFeatureLevel::Type FeatureLevel)
{
	const FVertexFactoryType* VertexFactoryType;
	FVertexDeclarationElementList VertexElements;
	if (LODRenderData.SkinWeightVertexBuffer.GetBoneInfluenceType() == GPUSkinBoneInfluenceType::DefaultBoneInfluence)
	{
		VertexFactoryType = &TGPUSkinAPEXClothVertexFactory<GPUSkinBoneInfluenceType::DefaultBoneInfluence>::StaticType;
		TGPUSkinAPEXClothVertexFactory<GPUSkinBoneInfluenceType::DefaultBoneInfluence>::GetVertexElements(FeatureLevel, EVertexInputStreamType::Default, GPUSkinDataType, VertexElements);
	}
	else
	{
		VertexFactoryType = &TGPUSkinAPEXClothVertexFactory<GPUSkinBoneInfluenceType::UnlimitedBoneInfluence>::StaticType;
		TGPUSkinAPEXClothVertexFactory<GPUSkinBoneInfluenceType::UnlimitedBoneInfluence>::GetVertexElements(FeatureLevel, EVertexInputStreamType::Default, GPUSkinDataType, VertexElements);
	}
	return FPSOPrecacheVertexFactoryData(VertexFactoryType, VertexElements);
}

/**
 * Creates a vertex factory entry for the given type and initialize it on the render thread
 */
static void CreateVertexFactoryCloth(
	FRHICommandList& RHICmdList,
	TArray<TUniquePtr<FGPUBaseSkinVertexFactory>>& VertexFactories,
	TUniquePtr<FGPUSkinPassthroughVertexFactory>* PassthroughVertexFactory,
	const FSkeletalMeshObjectGPUSkin::FVertexFactoryBuffers& VertexBuffers,
	ERHIFeatureLevel::Type FeatureLevel,
	FGPUSkinPassthroughVertexFactory::EVertexAttributeFlags VertexAttributeMask,
	uint32 NumBones,
	uint32 BoneOffset,
	uint32 BaseVertexIndex,
	uint32 NumInfluencesPerVertex,
	bool bUsedForPassthroughVertexFactory,
	bool bUnifiedBoneBuffer)
{
	FGPUBaseSkinVertexFactory* VertexFactory = nullptr;
	GPUSkinBoneInfluenceType BoneInfluenceType = VertexBuffers.SkinWeightVertexBuffer->GetBoneInfluenceType();
	const FGPUBaseSkinVertexFactory::FInitializer Initializer
	{
		  .FeatureLevel    = FeatureLevel
		, .NumBones        = NumBones
		, .BoneOffset      = BoneOffset
		, .NumVertices     = VertexBuffers.NumVertices
		, .BaseVertexIndex = BaseVertexIndex
		, .bUsedForPassthroughVertexFactory = bUsedForPassthroughVertexFactory
		, .bUnifiedBoneBuffer = bUnifiedBoneBuffer
	};

	if (BoneInfluenceType == GPUSkinBoneInfluenceType::DefaultBoneInfluence)
	{
		VertexFactory = new TGPUSkinAPEXClothVertexFactory<GPUSkinBoneInfluenceType::DefaultBoneInfluence>(Initializer, NumInfluencesPerVertex);
	}
	else
	{
		VertexFactory = new TGPUSkinAPEXClothVertexFactory<GPUSkinBoneInfluenceType::UnlimitedBoneInfluence>(Initializer, NumInfluencesPerVertex);
	}
	VertexFactories.Add(TUniquePtr<FGPUBaseSkinVertexFactory>(VertexFactory));

	FDynamicUpdateVertexFactoryData VertexUpdateData(VertexFactory, VertexBuffers);

	// update vertex factory components and sync it
	FGPUSkinAPEXClothDataType Data;
	InitGPUSkinVertexFactoryComponents(&Data, VertexUpdateData.VertexBuffers, VertexUpdateData.VertexFactory);
	InitAPEXClothVertexFactoryComponents(&Data, VertexUpdateData.VertexBuffers);
	VertexUpdateData.VertexFactory->SetData(RHICmdList, &Data);
	VertexUpdateData.VertexFactory->InitResource(RHICmdList);

	InitPassthroughVertexFactory_RenderThread(RHICmdList, PassthroughVertexFactory, VertexUpdateData.VertexFactory, FeatureLevel, VertexAttributeMask);
}

void FSkeletalMeshObjectGPUSkin::GetUsedVertexFactoryData(
	FSkeletalMeshRenderData* SkelMeshRenderData,
	int32 LODIndex,
	USkinnedMeshComponent* SkinnedMeshComponent,
	FSkelMeshRenderSection& RenderSection,
	ERHIFeatureLevel::Type InFeatureLevel,
	FPSOPrecacheVertexFactoryDataList& VertexFactoryDataList)
{
	FSkeletalMeshLODRenderData& LODRenderData = SkelMeshRenderData->LODRenderData[LODIndex];

	FSkelMeshComponentLODInfo* CompLODInfo = nullptr;
	if (SkinnedMeshComponent && SkinnedMeshComponent->LODInfo.IsValidIndex(LODIndex))
	{
		CompLODInfo = &SkinnedMeshComponent->LODInfo[LODIndex];
	}

	const ESkeletalMeshGPUSkinTechnique GPUSkinTechnique = ::GetGPUSkinTechnique(SkinnedMeshComponent, SkelMeshRenderData, LODIndex, InFeatureLevel);

	// Setup tmp MeshObjectLOD object to extract the vertex factory buffers
	FSkeletalMeshObjectGPUSkin::FSkeletalMeshObjectLOD SkeletalMeshObjectLOD(SkelMeshRenderData, LODIndex, InFeatureLevel, nullptr, GPUSkinTechnique);
	SkeletalMeshObjectLOD.MeshObjectWeightBuffer = FSkeletalMeshObject::GetSkinWeightVertexBuffer(LODRenderData, CompLODInfo);
	SkeletalMeshObjectLOD.MeshObjectColorBuffer = FSkeletalMeshObject::GetColorVertexBuffer(LODRenderData, CompLODInfo);

	// Vertex buffers available for the LOD
	FVertexFactoryBuffers VertexBuffers;
	SkeletalMeshObjectLOD.GetVertexBuffers(VertexBuffers, LODRenderData, EBoneTransformStorageMode::Direct);

	// Setup the skin data type so the correct vertex element data can be collected
	FGPUSkinDataType GPUSkinDataType;
	InitGPUSkinVertexFactoryComponents(&GPUSkinDataType, VertexBuffers, nullptr /*FGPUBaseSkinVertexFactory*/);

	if (GPUSkinTechnique != ESkeletalMeshGPUSkinTechnique::Inline || FGPUSkinCache::IsGPUSkinCacheRayTracingSupported(InFeatureLevel))
	{
		const FVertexFactoryType* GPUSkinVFType = &FGPUSkinPassthroughVertexFactory::StaticType;
		bool bSupportsManualVertexFetch = GPUSkinVFType->SupportsManualVertexFetch(GMaxRHIFeatureLevel);
		if (!bSupportsManualVertexFetch)
		{
			FVertexDeclarationElementList VertexElements;
			bool bOverrideColorVertexBuffer = false;
			FGPUSkinPassthroughVertexFactory::FDataType Data;
			LODRenderData.StaticVertexBuffers.InitComponentVF(nullptr /*VertexFactory*/, 0, bOverrideColorVertexBuffer, Data);
			FGPUSkinPassthroughVertexFactory::GetVertexElements(GMaxRHIFeatureLevel, EVertexInputStreamType::Default, bSupportsManualVertexFetch, Data, VertexElements);
			VertexFactoryDataList.AddUnique(FPSOPrecacheVertexFactoryData(GPUSkinVFType, VertexElements));
		}
		else
		{
			VertexFactoryDataList.AddUnique(FPSOPrecacheVertexFactoryData(&FGPUSkinPassthroughVertexFactory::StaticType));
		}
	}

	// Only add GPU skin cache vertex factory if run inline (not GPUSkinCache or MeshDeformer)
	if (GPUSkinTechnique == ESkeletalMeshGPUSkinTechnique::Inline)
	{
		// Add GPU skin cloth vertex factory type is needed
		const EShaderPlatform ShaderPlatform = GetFeatureLevelShaderPlatform(InFeatureLevel);
		const bool bClothEnabled = FGPUBaseSkinAPEXClothVertexFactory::IsClothEnabled(ShaderPlatform);
		if (bClothEnabled && RenderSection.HasClothingData())
		{
			VertexFactoryDataList.AddUnique(GetVertexFactoryDataCloth(LODRenderData, GPUSkinDataType, InFeatureLevel));
		}
		else
		{
			// Add GPU skin vertex factory type
			VertexFactoryDataList.AddUnique(GetVertexFactoryData(LODRenderData, GPUSkinDataType, InFeatureLevel));
		}
	}
}

/**
 * Determine the current vertex buffers valid for the current LOD
 *
 * @param OutVertexBuffers output vertex buffers
 */
void FSkeletalMeshObjectGPUSkin::FSkeletalMeshObjectLOD::GetVertexBuffers(FVertexFactoryBuffers& OutVertexBuffers, FSkeletalMeshLODRenderData& LODData, EBoneTransformStorageMode StorageMode)
{
	OutVertexBuffers.StaticVertexBuffers = &LODData.StaticVertexBuffers;
	OutVertexBuffers.ColorVertexBuffer = MeshObjectColorBuffer;
	OutVertexBuffers.SkinWeightVertexBuffer = MeshObjectWeightBuffer;
	OutVertexBuffers.MorphVertexBufferPool = MorphVertexBufferPool;
	OutVertexBuffers.APEXClothVertexBuffer = &LODData.ClothVertexBuffer;
	OutVertexBuffers.NumVertices = LODData.GetNumVertices();
}

/** 
 * Init vertex factory resources for this LOD 
 *
 * @param VertexBuffers - available vertex buffers to reference in vertex factory streams
 * @param Chunks - relevant chunk information (either original or from swapped influence)
 */
void FSkeletalMeshObjectGPUSkin::FVertexFactoryData::InitVertexFactories(
	FRHICommandList& RHICmdList,
	const FVertexFactoryBuffers& VertexBuffers,
	const TArray<FSkelMeshRenderSection>& Sections,
	ERHIFeatureLevel::Type InFeatureLevel,
	FGPUSkinPassthroughVertexFactory::EVertexAttributeFlags VertexAttributeMask,
	ESkeletalMeshGPUSkinTechnique GPUSkinTechnique,
	bool bUnifiedBoneBuffer)
{
	const EShaderPlatform ShaderPlatform = GetFeatureLevelShaderPlatform(InFeatureLevel);
	const bool bClothEnabled = FGPUBaseSkinAPEXClothVertexFactory::IsClothEnabled(ShaderPlatform);
	const bool bCreatePassthroughVFs = VertexAttributeMask != FGPUSkinPassthroughVertexFactory::EVertexAttributeFlags::None;
	const bool bUsedForPassthroughVertexFactory = GPUSkinTechnique != ESkeletalMeshGPUSkinTechnique::Inline;

	uint32 BoneOffset = 0;

	VertexFactories.Empty(Sections.Num());

	// Optionally create passthrough VFs
	TUniquePtr<FGPUSkinPassthroughVertexFactory>* Passthrough = nullptr;
	if (bCreatePassthroughVFs)
	{
		PassthroughVertexFactory = nullptr;
		Passthrough = &PassthroughVertexFactory;
	}

	for (const FSkelMeshRenderSection& Section : Sections)
	{
		const int32 NumBones = Section.BoneMap.Num();

		if (!Section.IsValid())
		{
			VertexFactories.Add(nullptr);
		}
		else if (Section.HasClothingData() && bClothEnabled)
		{
			constexpr int32 ClothLODBias = 0;
			const uint32 NumClothWeights = Section.ClothMappingDataLODs.Num() ? Section.ClothMappingDataLODs[ClothLODBias].Num(): 0;
			const uint32 NumPositionVertices = Section.NumVertices;
			// NumInfluencesPerVertex should be a whole integer
			check(NumClothWeights % NumPositionVertices == 0);
			const uint32 NumInfluencesPerVertex = NumClothWeights / NumPositionVertices;
			CreateVertexFactoryCloth(RHICmdList, VertexFactories, Passthrough, VertexBuffers, InFeatureLevel, VertexAttributeMask, NumBones, BoneOffset, Section.BaseVertexIndex, NumInfluencesPerVertex, bUsedForPassthroughVertexFactory, bUnifiedBoneBuffer);
		}
		else
		{
			CreateVertexFactory(RHICmdList, VertexFactories, Passthrough, VertexBuffers, InFeatureLevel, VertexAttributeMask, NumBones, BoneOffset, Section.BaseVertexIndex, bUsedForPassthroughVertexFactory, bUnifiedBoneBuffer);
		}

		BoneOffset += Section.IsValid() && !bUnifiedBoneBuffer ? Section.BoneMap.Num() : 0;
	}
}

void FSkeletalMeshObjectGPUSkin::FVertexFactoryData::ReleaseVertexFactories()
{
	for (auto& VertexFactory : VertexFactories)
	{
		if (VertexFactory.IsValid())
		{
			VertexFactory->ReleaseResource();
		}
	}

	if (PassthroughVertexFactory)
	{
		PassthroughVertexFactory->ReleaseResource();
	}
}

void FSkeletalMeshObjectGPUSkin::FVertexFactoryData::UpdateVertexFactoryData(const FVertexFactoryBuffers& VertexBuffers)
{
	UpdateVertexFactory(VertexFactories, VertexBuffers);
}

void FSkeletalMeshObjectGPUSkin::FSkeletalMeshObjectLOD::InitResources(
	FGPUSkinCache* InGPUSkinCache,
	const FSkelMeshObjectLODInfo& MeshLODInfo,
	const FSkelMeshComponentLODInfo* CompLODInfo,
	ERHIFeatureLevel::Type InFeatureLevel,
	FGPUSkinPassthroughVertexFactory::EVertexAttributeFlags VertexAttributeMask,
	EBoneTransformStorageMode InBoneTransformStorageMode,
	EBoneTransformAllocationMode InBoneTransformAllocationMode)
{
	check(SkelMeshRenderData);
	check(SkelMeshRenderData->LODRenderData.IsValidIndex(LODIndex));
	const bool bHasUnifiedBoneMap = SkelMeshRenderData->HasUnifiedBoneMap();

	// vertex buffer for each lod has already been created when skelmesh was loaded
	FSkeletalMeshLODRenderData& LODData = SkelMeshRenderData->LODRenderData[LODIndex];
	FVertexFactoryBuffers VertexBuffers;
	
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		UE::TScopeLock Lock(LODData.RenderDataMutex);
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
		
		MeshObjectWeightBuffer = FSkeletalMeshObject::GetSkinWeightVertexBuffer(LODData, CompLODInfo);
		MeshObjectColorBuffer = FSkeletalMeshObject::GetColorVertexBuffer(LODData, CompLODInfo);
		
		// Vertex buffers available for the LOD
		GetVertexBuffers(VertexBuffers, LODData, InBoneTransformStorageMode);
	}

	if (InBoneTransformAllocationMode == EBoneTransformAllocationMode::VertexFactory && bHasUnifiedBoneMap)
	{
		UnifiedBoneBuffer = MakeUnique<FUnifiedBoneBuffer>(SkelMeshRenderData->GetUnifiedBoneMap());
	}

	ENQUEUE_RENDER_COMMAND(FSkeletalMeshObjectLOD_InitResources)(UE::RenderCommandPipe::SkeletalMesh,
		[this, InGPUSkinCache, VertexBuffers, &LODData, InFeatureLevel, VertexAttributeMask, bHasUnifiedBoneMap](FRHICommandList& RHICmdList)
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		UE::TScopeLock Lock(LODData.RenderDataMutex);
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
				
		MeshObjectWeightBuffer_RenderThread = GetSkinWeightVertexBufferInfo(VertexBuffers.SkinWeightVertexBuffer);
		GPUSkinVertexFactories.InitVertexFactories(RHICmdList, VertexBuffers, LODData.RenderSections, InFeatureLevel, VertexAttributeMask, GPUSkinTechnique, bHasUnifiedBoneMap);

		if (DynamicBoundsStartOffset == INDEX_NONE && GPUSkinTechnique == ESkeletalMeshGPUSkinTechnique::GPUSkinCache)
		{
			DynamicBoundsNumSections = LODData.RenderSections.Num();
			DynamicBoundsStartOffset = InGPUSkinCache->AllocateDynamicMeshBoundsSlot(DynamicBoundsNumSections);
		}
	});
}

void FSkeletalMeshObjectGPUSkin::FSkeletalMeshObjectLOD::ReleaseResources(FGPUSkinCache* InGPUSkinCache)
{
	ENQUEUE_RENDER_COMMAND(FSkeletalMeshObjectLOD_ReleaseResources)(UE::RenderCommandPipe::SkeletalMesh,
		[this, InGPUSkinCache](FRHICommandList& RHICmdList)
	{
		GPUSkinVertexFactories.ReleaseVertexFactories();

		if (DynamicBoundsStartOffset != INDEX_NONE)
		{
			InGPUSkinCache->ReleaseDynamicMeshBoundsSlot(DynamicBoundsStartOffset, DynamicBoundsNumSections);
			DynamicBoundsStartOffset = INDEX_NONE;
		}

		UnifiedBoneBuffer = nullptr;
	});
}

void FSkeletalMeshObjectGPUSkin::FSkeletalMeshObjectLOD::InitMorphResources(const FSkelMeshObjectLODInfo& MeshLODInfo, ERHIFeatureLevel::Type InFeatureLevel)
{
	check(SkelMeshRenderData);
	check(SkelMeshRenderData->LODRenderData.IsValidIndex(LODIndex));

	// vertex buffer for each lod has already been created when skelmesh was loaded
	const FSkeletalMeshLODRenderData& LODData = SkelMeshRenderData->LODRenderData[LODIndex];

	// init the delta vertex buffer for this LOD
	const FName OwnerName = LODData.MorphTargetVertexInfoBuffers.GetOwnerName();

	// By design, we do not release MorphVertexBufferPool, as it may persist when render state gets re-created. Instead, it gets released
	// when its ref count goes to zero in the FSkeletalMeshObjectLOD destructor.
	MorphVertexBufferPool->InitResources(OwnerName);
}

TArray<FTransform>* FSkeletalMeshObjectGPUSkin::GetComponentSpaceTransforms() const
{
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	if(DynamicData)
	{
		return &(DynamicData->MeshComponentSpaceTransforms);
	}
	else
#endif
	{
		return NULL;
	}
}

TConstArrayView<FMatrix44f> FSkeletalMeshObjectGPUSkin::GetReferenceToLocalMatrices() const
{
	return DynamicData->ReferenceToLocal;
}

TConstArrayView<FMatrix44f> FSkeletalMeshObjectGPUSkin::GetPrevReferenceToLocalMatrices() const
{
	// Too many revisions between previous / current to be useful. Fall back to current.
	if (DynamicData->RevisionNumber - DynamicData->PreviousRevisionNumber > 1)
	{
		return DynamicData->ReferenceToLocal;
	}

	if (!DynamicData->PreviousReferenceToLocal.IsEmpty())
	{
		return DynamicData->PreviousReferenceToLocal;
	}
	
	if (PreviousDynamicData)
	{
		return PreviousDynamicData->ReferenceToLocal;
	}

	return {};
}

FSkeletalMeshObject::FBoneTransforms FSkeletalMeshObjectGPUSkin::GetBoneTransforms() const
{
	return FBoneTransforms
	{
		  .CurrentTransforms      = GetReferenceToLocalMatrices()
		, .PreviousTransforms     = GetPrevReferenceToLocalMatrices()
		, .CurrentRevisionNumber  = DynamicData->RevisionNumber
		, .PreviousRevisionNumber = DynamicData->PreviousRevisionNumber
		, .UpdateMode             = DynamicData->PreviousBoneTransformUpdateMode
	};
}

FSkeletalMeshObject::FBoneBufferSRV FSkeletalMeshObjectGPUSkin::GetSectionBoneBuffer(int32 InLODIndex, int32 InSectionIndex, bool bPreviousFrame) const
{
	FBoneBufferSRV Result;

	if (!DynamicData || !LODs.IsValidIndex(InLODIndex))
	{
		return Result;
	}

	const FGPUBaseSkinVertexFactory* BaseVertexFactory = GetBaseSkinVertexFactory(InLODIndex, InSectionIndex);
	if (!BaseVertexFactory)
	{
		return Result;
	}

	if (BoneTransformAllocationMode == EBoneTransformAllocationMode::Scene)
	{
		// Scene skinning extension path: shared transform buffer with combined offset.
		Result.SRV = SceneExtensionBoneBuffer.TransformBufferSRV;
		Result.Offset = SceneExtensionBoneBuffer.TransformBufferOffset
			+ ((bPreviousFrame ? 1u : 0u) ^ SceneExtensionBoneBuffer.CurrentTransformSlot) * SceneExtensionBoneBuffer.MaxTransformCount
			+ BaseVertexFactory->GetBoneOffset();
	}
	else
	{
		const FGPUBaseSkinVertexFactory::FShaderDataType& ShaderData = BaseVertexFactory->GetShaderData();
		if (ShaderData.HasBoneBufferForReading(bPreviousFrame))
		{
			Result.SRV = ShaderData.GetBoneBufferForReading(bPreviousFrame).VertexBufferSRV;
		}
	}

	return Result;
}

FMeshDeformerGeometry& FSkeletalMeshObjectGPUSkin::GetDeformerGeometry(int32 LODIndex)
{
	return LODs[LODIndex].DeformerGeometry;
}

bool FSkeletalMeshObjectGPUSkin::GetCachedGeometry(FRDGBuilder& GraphBuilder, FCachedGeometry& OutCachedGeometry) const
{
	OutCachedGeometry = FCachedGeometry();

	// Cached geometry is only available if we are using skin cache or a mesh deformer.	
	if (DynamicData == nullptr || DynamicData->GPUSkinTechnique == ESkeletalMeshGPUSkinTechnique::Inline)
	{
		return false;
	}

	const int32 LodIndex = GetLOD();
	if (SkeletalMeshRenderData == nullptr || !SkeletalMeshRenderData->LODRenderData.IsValidIndex(LodIndex))
	{
		return false;
	}

	FSkeletalMeshLODRenderData const& LODRenderData = SkeletalMeshRenderData->LODRenderData[LodIndex];
	const uint32 SectionCount = LODRenderData.RenderSections.Num();

	FVertexFactoryData const& VertexFactories = LODs[LodIndex].GPUSkinVertexFactories;
	if (VertexFactories.VertexFactories.Num() != SectionCount)
	{
		return false;
	}

	FGPUSkinCache::AddAsyncComputeWait(GraphBuilder);

	for (uint32 SectionIndex = 0; SectionIndex < SectionCount; ++SectionIndex)
	{
		FCachedGeometry::Section& CachedSection = OutCachedGeometry.Sections.AddDefaulted_GetRef();

		if (SkinCacheEntry != nullptr)
		{
			// Get the cached geometry SRVs from the skin cache.
			FRWBuffer* PositionBuffer = FGPUSkinCache::GetPositionBuffer(GraphBuilder, SkinCacheEntry, SectionIndex);
			if (!PositionBuffer || !PositionBuffer->SRV)
			{
				// Skip section which are not valid, but continue to fill in other section.
				// A section might be invalid because it is disabled/deactivated for instance, but the rest of the sections are still valid
				CachedSection = FCachedGeometry::Section();
				CachedSection.SectionIndex = SectionIndex;
				continue;
			}
			FRWBuffer* PreviousPositionBuffer = FGPUSkinCache::GetPreviousPositionBuffer(GraphBuilder, SkinCacheEntry, SectionIndex);

			CachedSection.PositionBuffer = PositionBuffer->SRV;
			CachedSection.PreviousPositionBuffer = PreviousPositionBuffer ? PreviousPositionBuffer->SRV : PositionBuffer->SRV;

			FRWBuffer* TangentBuffer = FGPUSkinCache::GetTangentBuffer(GraphBuilder, SkinCacheEntry, SectionIndex);
			CachedSection.TangentBuffer = TangentBuffer ? TangentBuffer->SRV.GetReference() : nullptr;
		}
		else
		{
			
			// Get the cached geometry SRVs from the deformer geometry.
			FMeshDeformerGeometry const& DeformerGeometry = LODs[LodIndex].DeformerGeometry;

			if (!DeformerGeometry.Position.IsValid())
			{
				// Reset all output if one section isn't available.
				OutCachedGeometry.Sections.Reset();
				return false;
			}

			CachedSection.PositionBuffer = DeformerGeometry.PositionSRV;
			CachedSection.PreviousPositionBuffer = DeformerGeometry.PrevPositionSRV;
			CachedSection.PreviousPositionBuffer = CachedSection.PreviousPositionBuffer != nullptr ? CachedSection.PreviousPositionBuffer : CachedSection.PositionBuffer;
		}

		CachedSection.IndexBuffer = LODRenderData.MultiSizeIndexContainer.GetIndexBuffer()->GetSRV();
		CachedSection.TotalIndexCount = LODRenderData.MultiSizeIndexContainer.GetIndexBuffer()->Num();
		CachedSection.TotalVertexCount = LODRenderData.StaticVertexBuffers.PositionVertexBuffer.GetNumVertices();
		CachedSection.UVsBuffer = LODRenderData.StaticVertexBuffers.StaticMeshVertexBuffer.GetTexCoordsSRV();
		CachedSection.UVsChannelOffset = 0; // Assume that we needs to pair meshes based on UVs 0
		CachedSection.UVsChannelCount = LODRenderData.StaticVertexBuffers.StaticMeshVertexBuffer.GetNumTexCoords();

		FSkelMeshRenderSection const& Section = LODRenderData.RenderSections[SectionIndex];
		CachedSection.LODIndex = LodIndex;
		CachedSection.SectionIndex = SectionIndex;
		CachedSection.NumPrimitives = Section.NumTriangles;
		CachedSection.NumVertices = Section.NumVertices;
		CachedSection.IndexBaseIndex = Section.BaseIndex;
		CachedSection.VertexBaseIndex = Section.BaseVertexIndex;
	}

	OutCachedGeometry.LODIndex = LodIndex;
	return true;
}

int32 FSkeletalMeshObjectGPUSkin::GetDynamicBoundsStartOffset(int32 LODIndex) const
{
	return LODs[LODIndex].DynamicBoundsStartOffset;
}

/*-----------------------------------------------------------------------------
FDynamicSkelMeshObjectDataGPUSkin
-----------------------------------------------------------------------------*/

int32 FDynamicSkelMeshObjectDataGPUSkin::Reset()
{
	bRecreating = false;
	RevisionNumber = INDEX_NONE;
	PreviousRevisionNumber = INDEX_NONE;
	LODIndex = 0;
	ActiveMorphTargets.Reset();
	MorphTargetWeights.Reset();
	ExternalMorphWeightData.Reset();
	ExternalMorphSets.Reset();
	NumWeightedActiveMorphTargets = 0;
	ClothingSimData.Reset();
	ClothBlendWeight = 0.0f;
	GPUSkinTechnique = ESkeletalMeshGPUSkinTechnique::Inline;
#if RHI_RAYTRACING
	bAnySegmentUsesWorldPositionOffset = false;
#endif

	PreviousReferenceToLocal.Empty();

	int32 Size = sizeof(FDynamicSkelMeshObjectDataGPUSkin);

	ReferenceToLocal.Reset();
	Size += ReferenceToLocal.GetAllocatedSize();

	ReferenceToLocalForRayTracing.Reset();
	Size += ReferenceToLocalForRayTracing.GetAllocatedSize();

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	MeshComponentSpaceTransforms.Reset();
	Size += MeshComponentSpaceTransforms.GetAllocatedSize();
#endif

	return Size;
}

void FDynamicSkelMeshObjectDataGPUSkin::InitDynamicSkelMeshObjectDataGPUSkin(
	const FSkinnedMeshSceneProxyDynamicData& InDynamicData,
	const FPrimitiveSceneProxy* InSceneProxy,
	const USkinnedAsset* InSkinnedAsset,
	FSkeletalMeshRenderData* InSkeletalMeshRenderData,
	FSkeletalMeshObjectGPUSkin* InMeshObject,
	int32 InLODIndex,
	const FMorphTargetWeightMap& InActiveMorphTargets,
	const TArray<float>& InMorphTargetWeights, 
	EPreviousBoneTransformUpdateMode InPreviousBoneTransformUpdateMode,
	const FExternalMorphWeightData& InExternalMorphWeightData)
{
	BoneTransformFrameNumber = InDynamicData.GetCurrentBoneTransformFrame();
	RevisionNumber = InDynamicData.GetBoneTransformRevisionNumber();
	PreviousRevisionNumber = InDynamicData.GetPreviousBoneTransformRevisionNumber();
	bRecreating = InDynamicData.IsRenderStateRecreating();
	PreviousBoneTransformUpdateMode = InPreviousBoneTransformUpdateMode;

	LODIndex = InLODIndex;
	check(!ActiveMorphTargets.Num() && !ReferenceToLocal.Num() && !ClothingSimData.Num() && !MorphTargetWeights.Num());

	// append instead of equals to avoid alloc
	MorphTargetWeights.Append(InMorphTargetWeights);
	NumWeightedActiveMorphTargets = 0;

	ExternalMorphWeightData = InExternalMorphWeightData;
	ExternalMorphWeightData.UpdateNumActiveMorphTargets();

	if (InDynamicData.IsValidExternalMorphSetLODIndex(InLODIndex))
	{
		ExternalMorphSets = InDynamicData.GetExternalMorphSets(InLODIndex);
	}	

	const FSkeletalMeshSceneProxy* SkeletalMeshProxy = static_cast<const FSkeletalMeshSceneProxy*>(InSceneProxy);

#if RHI_RAYTRACING
	int32 RayTracingLODBias = GetRayTracingSkeletalMeshGlobalLODBias();
	// TODO: MeshDeformer only supports using the same LOD as rendering so we have to disable ray tracing LOD bias
	if (InMeshObject->GetGPUSkinTechnique(LODIndex) == ESkeletalMeshGPUSkinTechnique::MeshDeformer)
	{
		RayTracingLODBias = 0;
	}
	// If Proxy is not visible in raytracing scene, set RayTracingLODIndex to -1 which means that additional RT update will not be performed for that mesh object, probably RT proxy handles that 
	const bool bVisibleInRayTracing = SkeletalMeshProxy ? SkeletalMeshProxy->IsVisibleInRayTracing() : true;
	RayTracingLODIndex = bVisibleInRayTracing ? FMath::Clamp(FMath::Max(LODIndex + RayTracingLODBias, InMeshObject->RayTracingMinLOD), LODIndex, InSkeletalMeshRenderData->LODRenderData.Num() - 1) : -1;
#endif

	// update ReferenceToLocal
	if (InDynamicData.bUseReferenceToLocalMatrices)
	{
		ReferenceToLocal = InDynamicData.ReferenceToLocalMatrices;
#if RHI_RAYTRACING
		if (bVisibleInRayTracing && RayTracingLODIndex != LODIndex)
		{
			ReferenceToLocalForRayTracing = InDynamicData.ReferenceToLocalMatrices;

			// It's possible that RayTracingLODIndex will reference bones that were unused by LODIndex. So we run an FK on unused bones in LODIndex.
			UpdateBonesRemovedByLOD(InMeshObject, ReferenceToLocalForRayTracing, InDynamicData, InSkinnedAsset, LODIndex, false);
		}
#endif

		switch(PreviousBoneTransformUpdateMode)
		{
		case EPreviousBoneTransformUpdateMode::None:
			// otherwise, clear it, it will use previous buffer
			PreviousReferenceToLocal.Reset();
			break;
		case EPreviousBoneTransformUpdateMode::UpdatePrevious:
			PreviousReferenceToLocal = InDynamicData.PreviousReferenceToLocalMatrices;
			break;
		case EPreviousBoneTransformUpdateMode::DuplicateCurrentToPrevious:
			PreviousReferenceToLocal = ReferenceToLocal;
			break;
		}
	}
	else
	{
		// Gather any bones referenced by shadow shapes
		const TArray<FBoneIndexType>* ExtraRequiredBoneIndices = nullptr;
		if (SkeletalMeshProxy && !SkeletalMeshProxy->IsNaniteMesh())
		{
			// TODO: Nanite-Skinning
			ExtraRequiredBoneIndices = &SkeletalMeshProxy->GetSortedShadowBoneIndices();
		}

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
		check(!MeshComponentSpaceTransforms.Num());
		// append instead of equals to avoid alloc
		MeshComponentSpaceTransforms.Append(InDynamicData.GetComponentSpaceTransforms());

		const bool bCalculateComponentSpaceTransformsFromLeader = MeshComponentSpaceTransforms.IsEmpty(); // This will be empty for follower components.
		TArray<FTransform>* const LeaderBoneMappedMeshComponentSpaceTransforms = bCalculateComponentSpaceTransformsFromLeader ? &MeshComponentSpaceTransforms : nullptr;
#else
		TArray<FTransform>* const LeaderBoneMappedMeshComponentSpaceTransforms = nullptr;
#endif
		
		UpdateRefToLocalMatrices( ReferenceToLocal, InDynamicData, InSkinnedAsset, InSkeletalMeshRenderData, LODIndex, ExtraRequiredBoneIndices, LeaderBoneMappedMeshComponentSpaceTransforms );
#if RHI_RAYTRACING
		if (bVisibleInRayTracing && RayTracingLODIndex != LODIndex)
		{
			ReferenceToLocalForRayTracing = ReferenceToLocal;

			// It's possible that RayTracingLODIndex will reference bones that were unused by LODIndex. So we run an FK on unused bones in LODIndex.
			UpdateBonesRemovedByLOD(InMeshObject, ReferenceToLocalForRayTracing, InDynamicData, InSkinnedAsset, LODIndex, false);
		}
#endif

		switch(PreviousBoneTransformUpdateMode)
		{
		case EPreviousBoneTransformUpdateMode::None:
			// otherwise, clear it, it will use previous buffer
			PreviousReferenceToLocal.Reset();
			break;
		case EPreviousBoneTransformUpdateMode::UpdatePrevious:
			UpdatePreviousRefToLocalMatrices(PreviousReferenceToLocal, InDynamicData, InSkinnedAsset, InSkeletalMeshRenderData, LODIndex, ExtraRequiredBoneIndices);
			break;
		case EPreviousBoneTransformUpdateMode::DuplicateCurrentToPrevious:
			PreviousReferenceToLocal = ReferenceToLocal;
			break;
		}
	}
	
	SectionIdsUseByActiveMorphTargets.Reset();

	// If we have external morph targets, just include all sections.
	if (ExternalMorphWeightData.HasActiveMorphs())
	{
		const FSkeletalMeshLODRenderData& LOD = InSkeletalMeshRenderData->LODRenderData[LODIndex];
		SectionIdsUseByActiveMorphTargets.SetNumUninitialized(LOD.RenderSections.Num(), EAllowShrinking::No);
		for (int32 Index = 0; Index < LOD.RenderSections.Num(); ++Index)
		{
			SectionIdsUseByActiveMorphTargets[Index] = Index;
		}
	}

	const float MorphTargetMaxBlendWeight = UE::SkeletalRender::Settings::GetMorphTargetMaxBlendWeight();

	// find number of morphs that are currently weighted and will affect the mesh
	ActiveMorphTargets.Reserve(InActiveMorphTargets.Num());
	for(const TTuple<const UMorphTarget*, int32>& MorphItem: InActiveMorphTargets)
	{
		const UMorphTarget* MorphTarget = MorphItem.Key;
		const int32 WeightIndex = MorphItem.Value;
		const float MorphTargetWeight = MorphTargetWeights[WeightIndex];
		const float MorphAbsWeight = FMath::Abs(MorphTargetWeight);

		if( MorphTarget != nullptr && 
			MorphAbsWeight >= MinMorphTargetBlendWeight &&
			MorphAbsWeight <= MorphTargetMaxBlendWeight &&
			MorphTarget->HasDataForLOD(LODIndex) ) 
		{
			NumWeightedActiveMorphTargets++;
			const TArray<int32>& MorphSectionIndices = MorphTarget->GetMorphLODModels()[LODIndex].SectionIndices;
			for (int32 SecId = 0; SecId < MorphSectionIndices.Num(); ++SecId)
			{
				SectionIdsUseByActiveMorphTargets.AddUnique(MorphSectionIndices[SecId]);
			}

			ActiveMorphTargets.Add(MorphTarget, WeightIndex);
		}
	}

	// Update the clothing simulation mesh positions and normals
	if (const IClothSimulationDataProvider* Provider = InDynamicData.GetClothSimulationDataProvider())
	{
		Provider->GetUpdateClothSimulationData_AnyThread(ClothingSimData, ClothObjectLocalToWorld, ClothBlendWeight);
	}
	else
	{
		ClothingSimData.Reset();
		ClothObjectLocalToWorld = FMatrix::Identity;
		ClothBlendWeight = 0.f;
	}

	GPUSkinTechnique = InMeshObject->GetGPUSkinTechnique(LODIndex);

	if (GPUSkinTechnique != ESkeletalMeshGPUSkinTechnique::MeshDeformer && InDynamicData.GetMeshDeformerInstanceForLOD(LODIndex) != nullptr)
	{
		UE_LOGF(LogSkeletalGPUSkinMesh, Fatal,
			"Skeletal mesh %ls, LOD %d is not set to use the mesh deformer skin technique, but the component deformer instance is valid. " "This means a mesh deformer was added but the skeletal mesh object was not recreated.",
			*InDynamicData.GetFName().ToString(), LODIndex);
	}

	if (!IsSkeletalMeshClothBlendEnabled())
	{
		ClothBlendWeight = 0.f;
	}

#if RHI_RAYTRACING
	if (bVisibleInRayTracing && SkeletalMeshProxy && !SkeletalMeshProxy->IsNaniteMesh())
	{
		// TODO: Nanite-Skinning
		bAnySegmentUsesWorldPositionOffset = SkeletalMeshProxy->bAnySegmentUsesWorldPositionOffset;
	}
#endif
}

bool FDynamicSkelMeshObjectDataGPUSkin::ActiveMorphTargetsEqual(
	const FMorphTargetWeightMap& InCompareActiveMorphTargets,
	const TArray<float>& CompareMorphTargetWeights
	) const
{
	if (InCompareActiveMorphTargets.Num() != ActiveMorphTargets.Num())
	{
		return false;
	}

	for(const TTuple<const UMorphTarget*, int32>& MorphItem: ActiveMorphTargets)
	{
		const UMorphTarget* MorphTarget = MorphItem.Key;
		const int32 WeightIndex = MorphItem.Value;
		const int32* CompareWeightIndex = InCompareActiveMorphTargets.Find(MorphTarget);
		if (CompareWeightIndex == nullptr)
		{
			return false;
		}

		if( FMath::Abs(MorphTargetWeights[WeightIndex] - CompareMorphTargetWeights[*CompareWeightIndex]) >= GMorphTargetWeightThreshold)
		{
			return false;
		}
	}
	return true;
}