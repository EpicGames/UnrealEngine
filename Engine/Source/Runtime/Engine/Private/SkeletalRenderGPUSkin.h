// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	SkeletalRenderGPUSkin.h: GPU skinned mesh object and resource definitions
=============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "ProfilingDebugging/ResourceSize.h"
#include "RenderResource.h"
#include "RayTracingGeometry.h"
#include "ShaderParameters.h"
#include "Components/ExternalMorphSet.h"
#include "Components/SkinnedMeshComponent.h"
#include "GlobalShader.h"
#include "GPUSkinVertexFactory.h"
#include "SkeletalMeshUpdater.h"
#include "SkeletalRenderPublic.h"
#include "ClothingSystemRuntimeTypes.h"
#include "Rendering/SkeletalMeshRenderData.h"
#include "Rendering/SkeletalMeshLODRenderData.h"
#include "Animation/MeshDeformerGeometry.h"
#include "SkinnedMeshSceneProxyDesc.h"
#include "InstancedSkeletalMeshObject.h"

enum class EGPUSkinCacheEntryMode;
class FGPUSkinCache;
class FSkeletalMeshUpdater;
class FSkeletalMeshObjectGPUSkin;
class FSkeletalMeshUpdatePacketGPUSkin;
class FVertexOffsetBuffers;

enum class ESkeletalMeshGPUSkinTechnique : uint8
{
	// Skinning is performed inline when the mesh is rendered in the mesh pass.
	Inline,

	// Skinning is performed by the GPU skin cache but falls back to the inline mode on a per-section basis if the skin cache is full.
	GPUSkinCache,

	// Skinning is performed by the mesh deformer graph.
	MeshDeformer
};

FSkinWeightVertexBufferInfo GetSkinWeightVertexBufferInfo(const FSkinWeightVertexBuffer* Buffer);

/** 
* Stores the updated matrices needed to skin the verts.
* Created by the game thread and sent to the rendering thread as an update 
*/
class FDynamicSkelMeshObjectDataGPUSkin final : public TPooledSkeletalMeshDynamicData<FDynamicSkelMeshObjectDataGPUSkin>
{
	friend class TSkeletalMeshDynamicDataPool<FDynamicSkelMeshObjectDataGPUSkin>;

	FDynamicSkelMeshObjectDataGPUSkin()
	{
		Reset();
	}

	int32 Reset();

public:
	/**
	* Constructor
	* Updates the ReferenceToLocal matrices using the new dynamic data.
	* @param	InSkelMeshComponent - parent skel mesh component
	* @param	InLODIndex - each lod has its own bone map 
	* @param	InActiveMorphTargets - morph targets active for the mesh
	* @param	InMorphTargetWeights - All morph target weights for the mesh
	*/
	void InitDynamicSkelMeshObjectDataGPUSkin(
		const FSkinnedMeshSceneProxyDynamicData& InDynamicData,
		const FPrimitiveSceneProxy* SceneProxy,
		const USkinnedAsset* InSkinnedAsset,
		FSkeletalMeshRenderData* InSkeletalMeshRenderData,
		FSkeletalMeshObjectGPUSkin* InMeshObject,
		int32 InLODIndex,
		const FMorphTargetWeightMap& InActiveMorphTargets,
		const TArray<float>& InMorphTargetWeights, 
		EPreviousBoneTransformUpdateMode PreviousBoneTransformUpdateMode,
		const FExternalMorphWeightData& InExternalMorphWeightData);

	/** ref pose to local space transforms */
	TArray<FMatrix44f> ReferenceToLocal;
	TArray<FMatrix44f> ReferenceToLocalForRayTracing;

	/** Previous ref pose to local space transform */
	TArray<FMatrix44f> PreviousReferenceToLocal;

	TConstArrayView<FMatrix44f> GetPreviousReferenceToLocal(EGPUSkinCacheEntryMode Mode) const;
	TConstArrayView<FMatrix44f> GetReferenceToLocal(EGPUSkinCacheEntryMode Mode) const;
	int32 GetLODIndex(EGPUSkinCacheEntryMode Mode) const;

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST) 
	/** component space bone transforms*/
	TArray<FTransform> MeshComponentSpaceTransforms;
#endif

	uint32 BoneTransformFrameNumber;
	uint32 RevisionNumber;
	uint32 PreviousRevisionNumber;
	EPreviousBoneTransformUpdateMode PreviousBoneTransformUpdateMode;

	/** currently LOD for bones being updated */
#if RHI_RAYTRACING
	int32 RayTracingLODIndex;
#endif
	/** current morph targets active on this mesh */
	FMorphTargetWeightMap ActiveMorphTargets;
	/** All morph target weights on this mesh */
	TArray<float> MorphTargetWeights;
	/** All section ID impacted by active morph target on this mesh */
	TArray<int32> SectionIdsUseByActiveMorphTargets;
	TArray<int32> SectionIdsUseByActiveMorphTargetsForRayTracing;
	/** number of active morph targets with weights > 0 */
	int32 NumWeightedActiveMorphTargets;

	/** 
	 * The dynamic data for each external morph target set.
	 * This dynamic data contains things such as the weights for each set of external morph targets.
	 */
	FExternalMorphWeightData ExternalMorphWeightData;

	/** The external morph target sets for this specific LOD. */
	FExternalMorphSets ExternalMorphSets;

	/** data for updating cloth section */
	TMap<int32, FClothSimulData> ClothingSimData;

    /** store transform of the cloth object **/
    FMatrix ClothObjectLocalToWorld;

	/** a weight factor to blend between simulated positions and skinned positions */	
	float ClothBlendWeight;

	/**
	* Compare the given set of active morph targets with the current list to check if different
	* @param CompareActiveMorphTargets - array of morphs to compare
	* @param MorphTargetWeights - array of morphs weights to compare
	* @return true if both sets of active morphs are equal
	*/
	bool ActiveMorphTargetsEqual(const FMorphTargetWeightMap& InCompareActiveMorphTargets, const TArray<float>& CompareMorphTargetWeights) const;

	bool IsRequiredUpdate() const
	{
		return PreviousBoneTransformUpdateMode != EPreviousBoneTransformUpdateMode::None;
	}

	/** Returns the size of memory allocated by render data */
	void GetResourceSizeEx(FResourceSizeEx& CumulativeResourceSize)
	{
		CumulativeResourceSize.AddDedicatedSystemMemoryBytes(sizeof(*this));
		
		CumulativeResourceSize.AddUnknownMemoryBytes(ReferenceToLocal.GetAllocatedSize());
		CumulativeResourceSize.AddUnknownMemoryBytes(ActiveMorphTargets.GetAllocatedSize());
	}

	static bool IsMorphUpdateNeeded(const FDynamicSkelMeshObjectDataGPUSkin* Previous, const FDynamicSkelMeshObjectDataGPUSkin* Current);

	/** The skinning technique to use for this mesh LOD. */
	ESkeletalMeshGPUSkinTechnique GPUSkinTechnique;

#if RHI_RAYTRACING
	uint8 bAnySegmentUsesWorldPositionOffset : 1;
#endif

	uint8 bRecreating : 1;
};

/** morph target mesh data for a single vertex delta */
struct FMorphGPUSkinVertex
{
	// Changes to this struct must be reflected in MorphTargets.usf!
	FVector3f			DeltaPosition;
	FVector3f			DeltaTangentZ;

	FMorphGPUSkinVertex() {};
	
	/** Construct for special case **/
	FMorphGPUSkinVertex(const FVector3f& InDeltaPosition, const FVector3f& InDeltaTangentZ)
	{
		DeltaPosition = InDeltaPosition;
		DeltaTangentZ = InDeltaTangentZ;
	}
};

/**
* MorphTarget vertices which have been combined into single position/tangentZ deltas
*/
class FMorphVertexBuffer final : public FVertexBuffer
{
public:
	/**
	* Default Constructor
	*/
	FMorphVertexBuffer()
		: bHasBeenUpdated(false)
		, bNeedsInitialClear(true)
		, bUsesComputeShader(false)
		, LODIdx(-1)
		, SkelMeshRenderData(nullptr)
	{
	}

	/** 
	* Constructor
	* @param	InSkelMeshRenderData	- render data containing the data for each LOD
	* @param	InLODIdx				- index of LOD model to use from the parent mesh
	*/
	FMorphVertexBuffer(FSkeletalMeshRenderData* InSkelMeshRenderData, int32 InLODIdx, ERHIFeatureLevel::Type InFeatureLevel)
		:	bHasBeenUpdated(false)	
		,	bNeedsInitialClear(true)
		,	LODIdx(InLODIdx)
		,	FeatureLevel(InFeatureLevel)
		,	SkelMeshRenderData(InSkelMeshRenderData)
	{
		check(SkelMeshRenderData);
		check(SkelMeshRenderData->LODRenderData.IsValidIndex(LODIdx));
		bUsesComputeShader = false;
	}
	/** 
	* Initialize the dynamic RHI for this rendering resource 
	*/
	virtual void InitRHI(FRHICommandListBase& RHICmdList) override;

	/** 
	* Release the dynamic RHI for this rendering resource 
	*/
	virtual void ReleaseRHI() override;

	inline void RecreateResourcesIfRequired(FRHICommandListBase& RHICmdList, bool bInUsesComputeShader)
	{
		if (bUsesComputeShader != bInUsesComputeShader)
		{
			UpdateRHI(RHICmdList);
		}
	}

	/** 
	* Morph target vertex name 
	*/
	FString GetFriendlyName() const { return TEXT("Morph target mesh vertices"); }

	/**
	 * Get Resource Size : mostly copied from InitRHI - how much they allocate when initialize
	 */
	SIZE_T GetResourceSize() const
	{
		SIZE_T ResourceSize = sizeof(*this);

		if (VertexBufferRHI)
		{
			// LOD of the skel mesh is used to find number of vertices in buffer
			FSkeletalMeshLODRenderData& LodData = SkelMeshRenderData->LODRenderData[LODIdx];

			// Create the buffer rendering resource
			ResourceSize += LodData.GetNumVertices() * sizeof(FMorphGPUSkinVertex);
		}

		return ResourceSize;
	}
	
	SIZE_T GetNumVerticies() const
	{
		// LOD of the skel mesh is used to find number of vertices in buffer
		FSkeletalMeshLODRenderData& LodData = SkelMeshRenderData->LODRenderData[LODIdx];
		// Create the buffer rendering resource
		return LodData.GetNumVertices();
	}

	/** Has been updated or not by UpdateMorphVertexBuffer**/
	bool bHasBeenUpdated;

	/** DX12 cannot clear the buffer in InitRHI with UAV flag enables, we should really have a Zero initzialized flag instead**/
	bool bNeedsInitialClear;

	// @param guaranteed only to be valid if the vertex buffer is valid
	FRHIShaderResourceView* GetSRV() const
	{
		return SRVValue;
	}

	// @param guaranteed only to be valid if the vertex buffer is valid
	FRHIUnorderedAccessView* GetUAV() const
	{
		return UAVValue;
	}

	FSkeletalMeshLODRenderData* GetLODRenderData() const { return &SkelMeshRenderData->LODRenderData[LODIdx]; }
	
	// section ids that are using this Morph buffer
	TArray<int32> SectionIds;
protected:
	// guaranteed only to be valid if the vertex buffer is valid
	FShaderResourceViewRHIRef SRVValue;

	// guaranteed only to be valid if the vertex buffer is valid
	FUnorderedAccessViewRHIRef UAVValue;

	bool bUsesComputeShader;

private:
	/** index to the SkelMeshResource.LODModels */
	int32	LODIdx;

	ERHIFeatureLevel::Type FeatureLevel;

	// parent mesh containing the source data, never 0
	FSkeletalMeshRenderData* SkelMeshRenderData;

	friend class FMorphVertexBufferPool;
};

/**
* Pooled morph vertex buffers that store the vertex deltas.
*/
class FMorphVertexBufferPool : public FRefCountedObject
{
public:
	FMorphVertexBufferPool(FSkeletalMeshRenderData* InSkelMeshRenderData, int32 InLOD, ERHIFeatureLevel::Type InFeatureLevel)
	{
		MorphVertexBuffers[0] = FMorphVertexBuffer(InSkelMeshRenderData, InLOD, InFeatureLevel);
		MorphVertexBuffers[1] = FMorphVertexBuffer(InSkelMeshRenderData, InLOD, InFeatureLevel);
	}

	~FMorphVertexBufferPool()
	{
		// Note that destruction of this class must occur on the render thread if InitResources has been called!
		// This is normally pointed to by FSkeletalMeshObjectGPUSkin, which is defer deleted on the render thread.
		if (bInitializedResources)
		{
			ReleaseResources();
		}
	}

	void InitResources(FName OwnerName);
	void ReleaseResources();
	SIZE_T GetResourceSize() const;
	void EnableDoubleBuffer(FRHICommandListBase& RHICmdList);
	bool IsInitialized() const						{ return bInitializedResources; }
	bool IsDoubleBuffered() const					{ return bDoubleBuffer; }
	void SetUpdatedFrameNumber(uint32 FrameNumber)	{ UpdatedFrameNumber = FrameNumber; }
	uint32 GetUpdatedFrameNumber() const			{ return UpdatedFrameNumber; }
	void SetCurrentRevisionNumber(uint32 RevisionNumber);
	const FMorphVertexBuffer& GetMorphVertexBufferForReading(bool bPrevious) const;
	/**
	 * Non-asserting variant of GetMorphVertexBufferForReading. Returns nullptr if the underlying
	 * VertexBufferRHI isn't valid yet
	 */
	const FMorphVertexBuffer* TryGetMorphVertexBufferForReading(bool bPrevious) const;
	FMorphVertexBuffer& GetMorphVertexBufferForWriting();

private:
	/** Resolves which MorphVertexBuffers[] slot a reader should observe */
	uint32 GetReadingIndex(bool bPrevious) const;

	/** Vertex buffer that stores the morph target vertex deltas. */
	FMorphVertexBuffer MorphVertexBuffers[2];
	/** If data is preserved when recreating render state, resources will already be initialized, so we need a flag to track that. */
	bool bInitializedResources = false;
	/** whether to double buffer. If going through skin cache, then use single buffer; otherwise double buffer. */
	bool bDoubleBuffer = false;

	// 0 / 1 to index into MorphVertexBuffer
	uint32 CurrentBuffer = 0;
	// RevisionNumber Tracker
	uint32 PreviousRevisionNumber = 0;
	uint32 CurrentRevisionNumber = 0;
	// Frame number of the morph vertex buffer that is last updated
	uint32 UpdatedFrameNumber = 0;
};

/**
 * Render data for a GPU skinned mesh
 */
class FSkeletalMeshObjectGPUSkin final : public FSkeletalMeshObject
{
public:
	/** @param	InSkeletalMeshComponent - skeletal mesh primitive we want to render */
	FSkeletalMeshObjectGPUSkin(const USkinnedMeshComponent* InMeshComponent, FSkeletalMeshRenderData* InSkelMeshRenderData, ERHIFeatureLevel::Type InFeatureLevel);
	FSkeletalMeshObjectGPUSkin(const FSkinnedMeshSceneProxyDesc& InMeshDesc, FSkeletalMeshRenderData* InSkelMeshRenderData, ERHIFeatureLevel::Type InFeatureLevel);
	virtual ~FSkeletalMeshObjectGPUSkin();

	//~ Begin FSkeletalMeshObject Interface
	virtual void InitResources(const FSkinnedMeshSceneProxyDesc& InMeshDesc) override;
	virtual void ReleaseResources() override;
	virtual void Update(int32 LODIndex, const FSkinnedMeshSceneProxyDynamicData& InSkeletalMeshDynamicData, const FPrimitiveSceneProxy* InSceneProxy, const USkinnedAsset* InSkinnedAsset, const FMorphTargetWeightMap& InActiveMorphTargets, const TArray<float>& InMorphTargetWeights, EPreviousBoneTransformUpdateMode PreviousBoneTransformUpdateMode, const FExternalMorphWeightData& InExternalMorphWeightData) override;
	virtual const FVertexFactory* GetSkinVertexFactory(const FSceneView* View, int32 LODIndex,int32 ChunkIdx, ESkinVertexFactoryMode VFMode = ESkinVertexFactoryMode::Default) const override;
	virtual const FVertexFactory* GetStaticSkinVertexFactory(int32 LODIndex, int32 ChunkIdx, ESkinVertexFactoryMode VFMode) const override;
	virtual FSkinningSceneExtensionProxy* CreateSceneExtensionProxy(const USkinnedAsset* InSkinnedAsset, bool bAllowScaling) const override;
	virtual void UpdateSceneExtensionHeader(FRHICommandListBase& RHICmdList, const FSkinningHeader& SkinningHeader, FRHIShaderResourceView* TransformDataBufferSRV) const override;
	virtual TArray<FTransform>* GetComponentSpaceTransforms() const override;
	virtual TConstArrayView<FMatrix44f> GetReferenceToLocalMatrices() const override;
	virtual TConstArrayView<FMatrix44f> GetPrevReferenceToLocalMatrices() const override;
	FBoneTransforms GetBoneTransforms() const override;
	FBoneBufferSRV GetSectionBoneBuffer(int32 InLODIndex, int32 InSectionIndex, bool bPreviousFrame) const override;

	virtual bool GetCachedGeometry(FRDGBuilder& GraphBuilder, FCachedGeometry& OutCachedGeometry) const override;
	
	FMeshDeformerGeometry& GetDeformerGeometry(int32 LODIndex);

	virtual bool IsGPUSkinMesh() const override { return true; }

	virtual int32 GetDynamicBoundsStartOffset(int32 LODIndex) const override;

#if RHI_RAYTRACING
	/** Geometry for ray tracing. */
	FRayTracingGeometry RayTracingGeometry;	
	FRWBuffer RayTracingDynamicVertexBuffer;

	// GetRayTracingGeometry()->IsInitialized() is checked as a workaround for UE-92634. FSkeletalMeshSceneProxy's resources may have already been released, but proxy has not removed yet)
	FRayTracingGeometry* GetRayTracingGeometry() { return RayTracingGeometry.HasValidInitializer() && RayTracingGeometry.IsInitialized() ? & RayTracingGeometry : nullptr; }
	const FRayTracingGeometry* GetRayTracingGeometry() const { return RayTracingGeometry.HasValidInitializer() && RayTracingGeometry.IsInitialized() ? & RayTracingGeometry : nullptr; }

	/** Return the internal vertex buffer only when initialized otherwise used the shared vertex buffer - needs to be updated every frame */
	FRWBuffer* GetRayTracingDynamicVertexBuffer() { return RayTracingDynamicVertexBuffer.NumBytes > 0 ? &RayTracingDynamicVertexBuffer : nullptr; }

	virtual int32 GetRayTracingLOD() const override
	{
		if (DynamicData)
		{
			return DynamicData->RayTracingLODIndex;
		}
		else
		{
			return 0;
		}
	}

	/** 
	 * Directly update ray tracing geometry. 
	 * This is quicker than the generic dynamic VSinCS path. 
	 * VSinCS path is still required for world position offset materials but this can still use 
	 * the updated vertex buffers from here with a passthrough vertex factory.
	 */
	virtual void UpdateRayTracingGeometry(FRHICommandListBase& RHICmdList, FSkeletalMeshLODRenderData& LODModel, uint32 LODIndex, TArray<FBufferRHIRef>& VertexBuffers) override;
	
#endif // RHI_RAYTRACING

	virtual int32 GetLOD() const override
	{
		if(DynamicData)
		{
			return DynamicData->LODIndex;
		}
		else
		{
			return 0;
		}
	}

	virtual bool HaveValidDynamicData() const override
	{ 
		return ( DynamicData!=NULL ); 
	}

	virtual void GetResourceSizeEx(FResourceSizeEx& CumulativeResourceSize) override
	{
		CumulativeResourceSize.AddDedicatedSystemMemoryBytes(sizeof(*this));
		
		if(DynamicData)
		{
			DynamicData->GetResourceSizeEx(CumulativeResourceSize);
		}
		
		CumulativeResourceSize.AddUnknownMemoryBytes(LODs.GetAllocatedSize()); 

		// include extra data from LOD
		for (int32 I=0; I<LODs.Num(); ++I)
		{
			LODs[I].GetResourceSizeEx(CumulativeResourceSize);
		}
	}
	//~ End FSkeletalMeshObject Interface

	/** Check if a given morph set is active or not. If so, we will process it. */
	bool IsExternalMorphSetActive(int32 MorphSetID, const FExternalMorphSet& MorphSet) const;

	FSkinWeightVertexBufferInfo GetSkinWeightVertexBuffer_RenderThread(int32 LODIndex) const override;

	/** 
	 * Get the skin vertex factory for direct skinning. 
	 * This is different from GetSkinVertexFactory because it ignores any passthrough vertex factories that may be in use.
	 */
	FGPUBaseSkinVertexFactory const* GetBaseSkinVertexFactory(int32 LODIndex, int32 ChunkIdx) const;

	/** 
	 * Vertex buffers that can be used for GPU skinning factories 
	 */
	struct FVertexFactoryBuffers
	{
		FStaticMeshVertexBuffers* StaticVertexBuffers = nullptr;
		const FSkinWeightVertexBuffer* SkinWeightVertexBuffer = nullptr;
		FColorVertexBuffer*	ColorVertexBuffer = nullptr;
		FMorphVertexBufferPool* MorphVertexBufferPool = nullptr;
		FSkeletalMeshVertexClothBuffer*	APEXClothVertexBuffer = nullptr;
		FVertexOffsetBuffers* VertexOffsetVertexBuffers = nullptr;
		uint32 NumVertices = 0;
	};

	virtual void RefreshClothingTransforms(const FMatrix& InNewLocalToWorld, uint32 FrameNumber) override;
	virtual void UpdateSkinWeightBuffer(const TArrayView<const FSkelMeshComponentLODInfo> InLODInfo) override;

	static void GetUsedVertexFactoryData(FSkeletalMeshRenderData* SkelMeshRenderData, int32 InLOD, USkinnedMeshComponent* SkinnedMeshComponent, FSkelMeshRenderSection& RenderSection, ERHIFeatureLevel::Type InFeatureLevel, FPSOPrecacheVertexFactoryDataList& VertexFactoryDataList);

	inline ESkeletalMeshGPUSkinTechnique GetGPUSkinTechnique(int32 LODIndex)
	{
		return LODs[LODIndex].GPUSkinTechnique;
	}

private:
	friend class FSkeletalMeshDeformerHelpers;
	
	friend class FSkeletalMeshObjectNanite;
	static void CreateVertexFactory(
		FRHICommandList& RHICmdList,
		TArray<TUniquePtr<FGPUBaseSkinVertexFactory>>& VertexFactories,
		TUniquePtr<FGPUSkinPassthroughVertexFactory>* PassthroughVertexFactory,
		const FSkeletalMeshObjectGPUSkin::FVertexFactoryBuffers& VertexBuffers,
		ERHIFeatureLevel::Type FeatureLevel,
		FGPUSkinPassthroughVertexFactory::EVertexAttributeFlags VertexAttributeMask,
		uint32 NumBones,
		uint32 BoneOffset,
		uint32 BaseVertexIndex,
		bool bUsedForPassthroughVertexFactory,
		bool bUnifiedBoneBuffer);

	static void UpdateRayTracingGeometry_Internal(
		FSkeletalMeshLODRenderData& LODModel, uint32 LODIndex, TArray<FBufferRHIRef>& VertexBuffers,
		FRayTracingGeometry& RayTracingGeometry, bool bAnySegmentUsesWorldPositionOffset, FSkeletalMeshObject* MeshObject);

	/**
	 * Vertex factories and their matrix arrays
	 */
	class FVertexFactoryData
	{
	public:
		FVertexFactoryData() {}

		/** one vertex factory for each chunk */
		TArray<TUniquePtr<FGPUBaseSkinVertexFactory>> VertexFactories;

		/** one passthrough vertex factory for each chunk */
		TUniquePtr<FGPUSkinPassthroughVertexFactory> PassthroughVertexFactory;

		uint32 NumVertices;

		/**
		 * Init default vertex factory resources for this LOD 
		 *
		 * @param VertexBuffers - available vertex buffers to reference in vertex factory streams
		 * @param Sections - relevant section information (either original or from swapped influence)
		 * @param VertexAttributeMask - the mask of vertex attributes that can be written to by the passthrough vertex factory
		 */
		void InitVertexFactories(
			FRHICommandList& RHICmdList,
			const FVertexFactoryBuffers& VertexBuffers,
			const TArray<FSkelMeshRenderSection>& Sections,
			ERHIFeatureLevel::Type FeatureLevel,
			FGPUSkinPassthroughVertexFactory::EVertexAttributeFlags VertexAttributeMask,
			ESkeletalMeshGPUSkinTechnique GPUSkinTechnique,
			bool bUnifiedBoneMap);

		void ReleaseVertexFactories();

		/** Refreshes the VertexFactor::FDataType to rebind any vertex buffers */
		void UpdateVertexFactoryData(const FVertexFactoryBuffers& VertexBuffers);

		uint64 GetResourceSize() const
		{
			return VertexFactories.GetAllocatedSize();
		}

	private:
		FVertexFactoryData(const FVertexFactoryData&);
		FVertexFactoryData& operator=(const FVertexFactoryData&);
	};

	/** vertex data for rendering a single LOD */
	struct FSkeletalMeshObjectLOD
	{
		FSkeletalMeshObjectLOD(
			FSkeletalMeshRenderData* InSkelMeshRenderData,
			int32 InLOD,
			ERHIFeatureLevel::Type InFeatureLevel,
			FMorphVertexBufferPool* InRecreateBufferPool,
			ESkeletalMeshGPUSkinTechnique InSkinTechnique);

		/** 
		 * Init rendering resources for this LOD 
		 * @param MeshLODInfo - information about the state of the bone influence swapping
		 * @param CompLODInfo - information about this LOD from the skeletal component 
		 */
		void InitResources(
			FGPUSkinCache* GPUSkinCache,
			const FSkelMeshObjectLODInfo& MeshLODInfo,
			const FSkelMeshComponentLODInfo* CompLODInfo,
			ERHIFeatureLevel::Type FeatureLevel,
			FGPUSkinPassthroughVertexFactory::EVertexAttributeFlags VertexAttributeMask,
			EBoneTransformStorageMode BoneTransformStorageMode,
			EBoneTransformAllocationMode InBoneTransformAllocationMode);

		/** 
		 * Release rendering resources for this LOD 
		 */
		void ReleaseResources(FGPUSkinCache* GPUSkinCache);

		/** 
		 * Init rendering resources for the morph stream of this LOD
		 * @param MeshLODInfo - information about the state of the bone influence swapping
		 * @param Chunks - relevant chunk information (either original or from swapped influence)
		 */
		void InitMorphResources(const FSkelMeshObjectLODInfo& MeshLODInfo, ERHIFeatureLevel::Type FeatureLevel);

		/**
		 * @return memory in bytes of size of the resources for this LOD
		 */
		void GetResourceSizeEx(FResourceSizeEx& CumulativeResourceSize)
		{
			CumulativeResourceSize.AddUnknownMemoryBytes(MorphVertexBufferPool->GetResourceSize());
			CumulativeResourceSize.AddUnknownMemoryBytes(GPUSkinVertexFactories.GetResourceSize());
		}

		FSkeletalMeshRenderData* SkelMeshRenderData;
		// index into FSkeletalMeshRenderData::LODRenderData[]
		int32 LODIndex;

		ERHIFeatureLevel::Type FeatureLevel;

		/** Unified bone buffer across the entire LOD when the mesh supports it. */
		TUniquePtr<FUnifiedBoneBuffer> UnifiedBoneBuffer;

		/** Pooled vertex buffers that store the morph target vertex deltas. */
		TRefCountPtr<FMorphVertexBufferPool> MorphVertexBufferPool;

		/** Default GPU skinning vertex factories and matrices */
		FVertexFactoryData GPUSkinVertexFactories;

		/** Skin weight buffer to use, could be from asset or component override */
		const FSkinWeightVertexBuffer* MeshObjectWeightBuffer = nullptr;
		FSkinWeightVertexBufferInfo MeshObjectWeightBuffer_RenderThread;

		/** Color buffer to user, could be from asset or component override */
		FColorVertexBuffer* MeshObjectColorBuffer = nullptr;

		/** Mesh deformer output buffers */
		FMeshDeformerGeometry DeformerGeometry;

		/** The preferred skinning technique to use for this mesh LOD. */
		ESkeletalMeshGPUSkinTechnique GPUSkinTechnique;

		/** The handle to a dynamic mesh bounds range for each section in the LOD when using skin cache. */
		int32 DynamicBoundsStartOffset = INDEX_NONE;
		int32 DynamicBoundsNumSections = 0;

		/**
		 * Update the contents of the morphtarget vertex buffer by accumulating all 
		 * delta positions and delta normals from the set of active morph targets
		 * @param ActiveMorphTargets - Morph to accumulate. assumed to be weighted and have valid targets
		 * @param MorphTargetWeights - All Morph weights
		 */
		void UpdateMorphVertexBufferCPU(FRHICommandList& RHICmdList, const FMorphTargetWeightMap& InActiveMorphTargets, const TArray<float>& MorphTargetWeights, const TArray<int32>& SectionIdsUseByActiveMorphTargets, 
										bool bGPUSkinCacheEnabled, FMorphVertexBuffer& MorphVertexBuffer);

		void UpdateMorphVertexBufferGPU(FRHICommandList& RHICmdList, const TArray<float>& MorphTargetWeights, const FMorphTargetVertexInfoBuffers& MorphTargetVertexInfoBuffers, 
										const TArray<int32>& SectionIdsUseByActiveMorphTargets, const FName& OwnerName, EGPUSkinCacheEntryMode Mode, FMorphVertexBuffer& MorphVertexBuffer,
										bool bClearMorphVertexBuffer, bool bNormalizePass, const FVector4& MorphScale, const FVector4& InvMorphScale);

		void UpdateSkinWeights(const FSkelMeshComponentLODInfo* CompLODInfo, EBoneTransformStorageMode BoneTransformStorageMode);

		/**
		 * Determine the current vertex buffers valid for this LOD
		 *
		 * @param OutVertexBuffers output vertex buffers
		 */
		void GetVertexBuffers(
			FVertexFactoryBuffers& OutVertexBuffers,
			FSkeletalMeshLODRenderData& LODData,
			EBoneTransformStorageMode StorageMode);

		// Temporary arrays used on UpdateMorphVertexBuffer(); these grow to the max and are not thread safe.
		static TArray<float> MorphAccumulatedWeightArray;
	};

	/** Render data for each LOD */
	TArray<struct FSkeletalMeshObjectLOD> LODs;

	/** Data that is updated dynamically and is needed for rendering */
	FDynamicSkelMeshObjectDataGPUSkin* DynamicData = nullptr;
	FDynamicSkelMeshObjectDataGPUSkin* PreviousDynamicData = nullptr;

	FSkeletalMeshObjectGPUSkin(const FSkeletalMeshObjectGPUSkin&);
	FSkeletalMeshObjectGPUSkin& operator=(const FSkeletalMeshObjectGPUSkin&);

	void InitMorphResources();

	void UpdateBufferData(FRHICommandList& RHICmdList, EGPUSkinCacheEntryMode Mode);
	void ProcessUpdatedDynamicData(FRHICommandList& RHICmdList, EGPUSkinCacheEntryMode Mode);
	void UpdateMorphVertexBuffer(FRHICommandList& RHICmdList);

	bool IsRayTracingSkinCacheUpdateNeeded() const
	{
#if RHI_RAYTRACING
		return DynamicData->RayTracingLODIndex != -1
			&& DynamicData->GPUSkinTechnique != ESkeletalMeshGPUSkinTechnique::MeshDeformer
			&& ShouldUseSeparateSkinCacheEntryForRayTracing()
			&& GetSkeletalMeshRenderData().bSupportRayTracing;
#else
		return false;
#endif
	}

	bool IsSkinCacheEnabled(EGPUSkinCacheEntryMode Mode) const;

	FSkeletalMeshUpdateHandle UpdateHandle;

	FGPUSkinCache* GPUSkinCache = nullptr;
	FMorphVertexBuffer* MorphVertexBuffer = nullptr;
	uint32 LastRayTracingBoneTransformUpdate = INDEX_NONE;
	EBoneTransformAllocationMode BoneTransformAllocationMode = EBoneTransformAllocationMode::VertexFactory;
	bool bMorphResourcesInitialized = false;
	bool bMorphNeedsUpdate = false;

	/** Cached scene extension bone data, set in UpdateSceneExtensionHeader, reset in ProcessUpdatedDynamicData. */
	mutable struct
	{
		FRHIShaderResourceView* TransformBufferSRV = nullptr;
		uint32 TransformBufferOffset = 0;
		uint32 MaxTransformCount = 0;
		uint32 CurrentTransformSlot = 0;

	} SceneExtensionBoneBuffer;

	friend FSkeletalMeshUpdater;
	friend FSkeletalMeshUpdatePacketGPUSkin;
};

class FGPUMorphUpdateCS : public FGlobalShader
{
public:
	DECLARE_SHADER_TYPE(FGPUMorphUpdateCS, Global);

	FGPUMorphUpdateCS();
	FGPUMorphUpdateCS(const ShaderMetaType::CompiledShaderInitializerType& Initializer);

	static const uint32 MorphTargetDispatchBatchSize = 128;

	void SetParameters(
		FRHIBatchedShaderParameters& BatchedParameters,
		const FVector4& LocalScale,
		const FMorphTargetVertexInfoBuffers& MorphTargetVertexInfoBuffers,
		FMorphVertexBuffer& MorphVertexBuffer,
		uint32 NumGroups,
		uint32 BatchOffsets[MorphTargetDispatchBatchSize],
		uint32 GroupOffsets[MorphTargetDispatchBatchSize],
		float Weights[MorphTargetDispatchBatchSize]);

	void Dispatch(FRHICommandList& RHICmdList, uint32 Size);
	void UnsetParameters(FRHIBatchedShaderUnbinds& BatchedUnbinds);

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters);

protected:
	LAYOUT_FIELD(FShaderResourceParameter, MorphVertexBufferParameter);

	LAYOUT_FIELD(FShaderParameter, MorphTargetWeightsParameter);
	LAYOUT_FIELD(FShaderParameter, OffsetAndSizeParameter);
	LAYOUT_FIELD(FShaderParameter, MorphTargetBatchOffsetsParameter);
	LAYOUT_FIELD(FShaderParameter, MorphTargetGroupOffsetsParameter);
	LAYOUT_FIELD(FShaderParameter, PositionScaleParameter);
	LAYOUT_FIELD(FShaderParameter, PrecisionParameter);
	LAYOUT_FIELD(FShaderParameter, NumGroupsParameter);

	LAYOUT_FIELD(FShaderResourceParameter, MorphDataBufferParameter);
};

class FGPUMorphNormalizeCS : public FGlobalShader
{
public:
	DECLARE_SHADER_TYPE(FGPUMorphNormalizeCS, Global);

	FGPUMorphNormalizeCS();
	FGPUMorphNormalizeCS(const ShaderMetaType::CompiledShaderInitializerType& Initializer);

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters);

	void SetParameters(FRHIBatchedShaderParameters& BatchedParameters, const FVector4& LocalScale, const FMorphTargetVertexInfoBuffers& MorphTargetVertexInfoBuffers, FMorphVertexBuffer& MorphVertexBuffer, uint32 NumVertices);

	void Dispatch(FRHICommandList& RHICmdList, uint32 NumVertices);
	void UnsetParameters(FRHIBatchedShaderUnbinds& BatchedUnbinds);

protected:
	LAYOUT_FIELD(FShaderResourceParameter, MorphVertexBufferParameter);

	LAYOUT_FIELD(FShaderParameter, PositionScaleParameter);
	LAYOUT_FIELD(FShaderParameter, NumVerticesParameter);
};