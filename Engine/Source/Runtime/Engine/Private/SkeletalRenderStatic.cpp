// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	SkeletalRenderStatic.cpp: CPU skinned skeletal mesh rendering code.
=============================================================================*/

#include "SkeletalRenderStatic.h"
#include "RenderUtils.h"
#include "Rendering/SkeletalMeshRenderData.h"
#include "Rendering/RenderCommandPipes.h"
#include "SkinnedMeshSceneProxyDesc.h"

#if RHI_RAYTRACING
#include "Engine/SkinnedAssetCommon.h"
#endif

FSkeletalMeshObjectStatic::FSkeletalMeshObjectStatic(const USkinnedMeshComponent* InMeshComponent, FSkeletalMeshRenderData* InSkelMeshRenderData, ERHIFeatureLevel::Type InFeatureLevel)
	: FSkeletalMeshObjectStatic(FSkinnedMeshSceneProxyDesc(InMeshComponent), InSkelMeshRenderData, InFeatureLevel) {}

FSkeletalMeshObjectStatic::FSkeletalMeshObjectStatic(const FSkinnedMeshSceneProxyDesc& InMeshDesc, FSkeletalMeshRenderData* InSkelMeshRenderData, ERHIFeatureLevel::Type InFeatureLevel)
	: FSkeletalMeshObject(InMeshDesc, InSkelMeshRenderData, InFeatureLevel)
{
	// create LODs to match the base mesh
	for (int32 LODIndex = 0; LODIndex < InSkelMeshRenderData->LODRenderData.Num(); LODIndex++)
	{
		new(LODs) FSkeletalMeshObjectLOD(InFeatureLevel, InSkelMeshRenderData, LODIndex);
	}

	InitResources(InMeshDesc);
	bSupportsStaticRelevance = true;
}

FSkeletalMeshObjectStatic::~FSkeletalMeshObjectStatic()
{
}

void FSkeletalMeshObjectStatic::InitResources(const FSkinnedMeshSceneProxyDesc& InMeshDesc)
{
	for( int32 LODIndex=0;LODIndex < LODs.Num();LODIndex++ )
	{
		FSkeletalMeshObjectLOD& SkelLOD = LODs[LODIndex];

		check(SkelLOD.SkelMeshRenderData);
		check(SkelLOD.SkelMeshRenderData->LODRenderData.IsValidIndex(LODIndex));
		
		// Skip LODs that have their render data stripped
		if (SkelLOD.SkelMeshRenderData->LODRenderData[LODIndex].GetNumVertices() > 0)
		{
			const FSkelMeshComponentLODInfo* CompLODInfo = InMeshDesc.LODInfo.IsValidIndex(LODIndex) ? &InMeshDesc.LODInfo[LODIndex] : nullptr;
			SkelLOD.InitResources(CompLODInfo);
		}
	}
}

void FSkeletalMeshObjectStatic::ReleaseResources()
{
	for( int32 LODIndex=0;LODIndex < LODs.Num();LODIndex++ )
	{
		FSkeletalMeshObjectLOD& SkelLOD = LODs[LODIndex];

		check(SkelLOD.SkelMeshRenderData);
		check(SkelLOD.SkelMeshRenderData->LODRenderData.IsValidIndex(LODIndex));
		
		// Skip LODs that have their render data stripped
		if (SkelLOD.SkelMeshRenderData->LODRenderData[LODIndex].GetNumVertices() > 0)
		{
			SkelLOD.ReleaseResources();
		}
	}
}

const FVertexFactory* FSkeletalMeshObjectStatic::GetSkinVertexFactory(const FSceneView* View, int32 LODIndex, int32 ChunkIdx, ESkinVertexFactoryMode VFMode) const
{
	return &LODs[LODIndex].VertexFactory; 
}

const FVertexFactory* FSkeletalMeshObjectStatic::GetStaticSkinVertexFactory(int32 LODIndex, int32 ChunkIdx, ESkinVertexFactoryMode VFMode) const
{
	return &LODs[LODIndex].VertexFactory;
}

TArray<FTransform>* FSkeletalMeshObjectStatic::GetComponentSpaceTransforms() const
{
	return nullptr;
}

TConstArrayView<FMatrix44f> FSkeletalMeshObjectStatic::GetReferenceToLocalMatrices() const
{
	return {};
}

int32 FSkeletalMeshObjectStatic::GetLOD() const
{
	// WorkingMinDesiredLODLevel and MinLODLevel could be a LOD that's not loaded, so we need to clamp it to the first loaded LOD in the render data.
	return FMath::Max3<int32>(WorkingMinDesiredLODLevel, MinLODLevel, SkeletalMeshRenderData->CurrentFirstLODIdx);
}

void FSkeletalMeshObjectStatic::FSkeletalMeshObjectLOD::InitResources(const FSkelMeshComponentLODInfo* CompLODInfo)
{
	check(SkelMeshRenderData);
	check(SkelMeshRenderData->LODRenderData.IsValidIndex(LODIndex));

	FSkeletalMeshLODRenderData& LODData = SkelMeshRenderData->LODRenderData[LODIndex];
	
	FPositionVertexBuffer* PositionVertexBufferPtr = &LODData.StaticVertexBuffers.PositionVertexBuffer;
	FStaticMeshVertexBuffer* StaticMeshVertexBufferPtr = &LODData.StaticVertexBuffers.StaticMeshVertexBuffer;
	
	// If we have a vertex color override buffer (and it's the right size) use it
	if (CompLODInfo &&
		CompLODInfo->OverrideVertexColors &&
		CompLODInfo->OverrideVertexColors->GetNumVertices() == PositionVertexBufferPtr->GetNumVertices())
	{
		ColorVertexBuffer = CompLODInfo->OverrideVertexColors;
	}
	else
	{
		ColorVertexBuffer = &LODData.StaticVertexBuffers.ColorVertexBuffer;
	}

	FLocalVertexFactory* VertexFactoryPtr = &VertexFactory;
	FColorVertexBuffer* ColorVertexBufferPtr = ColorVertexBuffer;

	ENQUEUE_RENDER_COMMAND(InitSkeletalMeshStaticSkinVertexFactory)(UE::RenderCommandPipe::SkeletalMesh,
		[VertexFactoryPtr, PositionVertexBufferPtr, StaticMeshVertexBufferPtr, ColorVertexBufferPtr](FRHICommandList& RHICmdList)
		{
			FLocalVertexFactory::FDataType Data;
			PositionVertexBufferPtr->InitResource(RHICmdList);
			StaticMeshVertexBufferPtr->InitResource(RHICmdList);
			ColorVertexBufferPtr->InitResource(RHICmdList);

			PositionVertexBufferPtr->BindPositionVertexBuffer(VertexFactoryPtr, Data);
			StaticMeshVertexBufferPtr->BindTangentVertexBuffer(VertexFactoryPtr, Data);
			StaticMeshVertexBufferPtr->BindPackedTexCoordVertexBuffer(VertexFactoryPtr, Data);
			StaticMeshVertexBufferPtr->BindLightMapVertexBuffer(VertexFactoryPtr, Data, 0);
			ColorVertexBufferPtr->BindColorVertexBuffer(VertexFactoryPtr, Data);

			VertexFactoryPtr->SetData(RHICmdList, Data);
			VertexFactoryPtr->InitResource(RHICmdList);
		});

#if RHI_RAYTRACING
	if (IsRayTracingEnabled() && SkelMeshRenderData->bSupportRayTracing)
	{
		SkelMeshRenderData->InitStaticRayTracingGeometry(LODIndex);
		bStaticRayTracingGeometryInitialized = true;
	}
#endif

	bResourcesInitialized = true;
}

/** 
 * Release rendering resources for this LOD 
 */
void FSkeletalMeshObjectStatic::FSkeletalMeshObjectLOD::ReleaseResources()
{	
#if RHI_RAYTRACING
	if (bStaticRayTracingGeometryInitialized)
	{
		SkelMeshRenderData->ReleaseStaticRayTracingGeometry(LODIndex);
		bStaticRayTracingGeometryInitialized = false;
	}
#endif

	BeginReleaseResource(&VertexFactory, &UE::RenderCommandPipe::SkeletalMesh);

	bResourcesInitialized = false;
}
