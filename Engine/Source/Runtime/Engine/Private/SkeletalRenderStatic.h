// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	SkeletalRenderStatic.h: Skinned mesh object rendered as static
=============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "ProfilingDebugging/ResourceSize.h"
#include "RenderResource.h"
#include "RayTracingGeometry.h"
#include "LocalVertexFactory.h"
#include "Components/SkinnedMeshComponent.h"
#include "SkeletalRenderPublic.h"

class FPrimitiveDrawInterface;
class UMorphTarget;

struct FSkinnedMeshSceneProxyDesc;

class FSkeletalMeshObjectStatic : public FSkeletalMeshObject
{
public:
	/** @param	InSkeletalMeshComponent - skeletal mesh primitive we want to render */
	ENGINE_API FSkeletalMeshObjectStatic(const USkinnedMeshComponent* InMeshComponent, FSkeletalMeshRenderData* InSkelMeshRenderData, ERHIFeatureLevel::Type InFeatureLevel);
	ENGINE_API FSkeletalMeshObjectStatic(const FSkinnedMeshSceneProxyDesc& InMeshDesc, FSkeletalMeshRenderData* InSkelMeshRenderData, ERHIFeatureLevel::Type InFeatureLevel);
	ENGINE_API virtual ~FSkeletalMeshObjectStatic();

	//~ Begin FSkeletalMeshObject Interface
	ENGINE_API virtual void InitResources(const FSkinnedMeshSceneProxyDesc& InMeshDesc) override;
	ENGINE_API virtual void ReleaseResources() override;
	virtual void Update(int32 LODIndex, const FSkinnedMeshSceneProxyDynamicData& InSkeletalMeshDynamicData, const FPrimitiveSceneProxy* InSceneProxy, const USkinnedAsset* InSkinnedAsset, const FMorphTargetWeightMap& InActiveMorphTargets, const TArray<float>& MorphTargetWeights, EPreviousBoneTransformUpdateMode PreviousBoneTransformUpdateMode, const FExternalMorphWeightData& InExternalMorphWeightData) override {};
	//virtual void UpdateRecomputeTangent(int32 MaterialIndex, int32 LODIndex, bool bRecomputeTangent) override {};
	virtual void EnableOverlayRendering(bool bEnabled, const TArray<int32>* InBonesOfInterest, const TArray<UMorphTarget*>* InMorphTargetOfInterest) override {};
	virtual bool IsCPUSkinned() const override { return true; }
	ENGINE_API virtual const FVertexFactory* GetSkinVertexFactory(const FSceneView* View, int32 LODIndex, int32 ChunkIdx, ESkinVertexFactoryMode VFMode = ESkinVertexFactoryMode::Default) const override;
	ENGINE_API virtual const FVertexFactory* GetStaticSkinVertexFactory(int32 LODIndex, int32 ChunkIdx, ESkinVertexFactoryMode VFMode) const override;
	ENGINE_API virtual TArray<FTransform>* GetComponentSpaceTransforms() const override;
	ENGINE_API virtual TConstArrayView<FMatrix44f> GetReferenceToLocalMatrices() const override;
	virtual int32 GetLOD() const override;
	//virtual const FTwoVectors& GetCustomLeftRightVectors(int32 SectionIndex) const override;
	virtual void DrawVertexElements(FPrimitiveDrawInterface* PDI, const FMatrix& ToWorldSpace, bool bDrawNormals, bool bDrawTangents, bool bDrawBinormals) const override {};
	virtual bool HaveValidDynamicData() const override
	{ 
		return false; 
	}

	virtual void GetResourceSizeEx(FResourceSizeEx& CumulativeResourceSize) override
	{
		CumulativeResourceSize.AddDedicatedSystemMemoryBytes(sizeof(*this));
		CumulativeResourceSize.AddDedicatedSystemMemoryBytes(LODs.GetAllocatedSize()); 
		// include extra data from LOD
		for (int32 I=0; I<LODs.Num(); ++I)
		{
			LODs[I].GetResourceSizeEx(CumulativeResourceSize);
		}
	}
	virtual void UpdateSkinWeightBuffer(const TArrayView<const FSkelMeshComponentLODInfo> InLODInfo) override {}

	//~ End FSkeletalMeshObject Interface

private:
	/** vertex data for rendering a single LOD */
	struct FSkeletalMeshObjectLOD
	{
		FSkeletalMeshRenderData* SkelMeshRenderData;
		// index into FSkeletalMeshResource::LODModels[]
		int32 LODIndex;
		FLocalVertexFactory	VertexFactory;

		/** Color buffer to user, could be from asset or component override */
		FColorVertexBuffer* ColorVertexBuffer;

		/**
		* Whether InitStaticRayTracingGeometry(...) was called during initialization,
		* so ReleaseStaticRayTracingGeometry(...) must be called when releasing resources.
		*/
		bool bStaticRayTracingGeometryInitialized;

		/** true if resources for this LOD have already been initialized. */
		bool bResourcesInitialized;

		FSkeletalMeshObjectLOD(ERHIFeatureLevel::Type InFeatureLevel, FSkeletalMeshRenderData* InSkelMeshRenderData, int32 InLOD)
			:	SkelMeshRenderData(InSkelMeshRenderData)
			,	LODIndex(InLOD)
			,	VertexFactory(InFeatureLevel, "FSkeletalMeshObjectStatic::FSkeletalMeshObjectLOD")
			,	ColorVertexBuffer(nullptr)
			,	bStaticRayTracingGeometryInitialized(false)
			,	bResourcesInitialized(false)
		{
		}

		/** 
		 * Init rendering resources for this LOD 
		 */
		void InitResources(const FSkelMeshComponentLODInfo* CompLODInfo);
		/** 
		 * Release rendering resources for this LOD 
		 */
		void ReleaseResources();

		void GetResourceSizeEx(FResourceSizeEx& CumulativeResourceSize)
 		{
			CumulativeResourceSize.AddDedicatedSystemMemoryBytes(sizeof(*this));
 		}
	};

	/** Render data for each LOD */
	TArray<FSkeletalMeshObjectLOD> LODs;
};

