// Copyright Epic Games, Inc. All Rights Reserved.

#include "Engine/SkinnedAsset.h"
#include "Engine/SkinnedAssetAsyncCompileUtils.h"
#include "Engine/SkinnedAssetCommon.h"
#include "Engine/World.h"
#include "PSOPrecache.h"
#include "SkinnedAssetCompiler.h"
#include "Animation/AnimStats.h"
#include "Materials/MaterialInterface.h"
#include "SkeletalRenderGPUSkin.h"
#include "PSOPrecache.h"
#include "AutoRTFM.h"
#if WITH_EDITORONLY_DATA
#include "Streaming/UVChannelDensity.h"
#endif
#include "ObjectCacheContext.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SkinnedAsset)

#if INTEL_ISPC
#include "SkinnedAsset.ispc.generated.h"
static_assert(sizeof(ispc::FTransform) == sizeof(FTransform), "sizeof(ispc::FTransform) != sizeof(FTransform)");
#endif

#if !defined(ANIM_SKINNED_ASSET_ISPC_ENABLED_DEFAULT)
#define ANIM_SKINNED_ASSET_ISPC_ENABLED_DEFAULT 1
#endif

// Support run-time toggling on supported platforms in non-shipping configurations
#if !INTEL_ISPC || UE_BUILD_SHIPPING
static constexpr bool bAnim_SkinnedAsset_ISPC_Enabled = INTEL_ISPC && ANIM_SKINNED_ASSET_ISPC_ENABLED_DEFAULT;
#else
static bool bAnim_SkinnedAsset_ISPC_Enabled = ANIM_SKINNED_ASSET_ISPC_ENABLED_DEFAULT;
static FAutoConsoleVariableRef CVarAnimSkinnedAssetISPCEnabled(TEXT("a.SkinnedAsset.ISPC"), bAnim_SkinnedAsset_ISPC_Enabled, TEXT("Whether to use ISPC optimizations on skinned assets"));
#endif

DEFINE_LOG_CATEGORY(LogSkinnedAsset)

static TAutoConsoleVariable<int32> CVarMemoryForSkinnedAssetCompile (
	TEXT("Memory.MemoryForSkinnedAssetCompile"),
	4608,		//	4.5 GiB
	TEXT("Memory in MiB set aside for a SkinnedAsset compile job\n"),
	ECVF_Default);

#if WITH_EDITORONLY_DATA
static void AccumulateUVDensities(float* OutWeightedUVDensities, float* OutWeights, const FSkeletalMeshLODRenderData& LODData, const FSkelMeshRenderSection& Section)
{
	const int32 NumTotalTriangles = LODData.GetTotalFaces();
	const int32 NumCoordinateIndex = FMath::Min<int32>(LODData.GetNumTexCoords(), TEXSTREAM_MAX_NUM_UVCHANNELS);

	FUVDensityAccumulator UVDensityAccs[TEXSTREAM_MAX_NUM_UVCHANNELS];
	for (int32 CoordinateIndex = 0; CoordinateIndex < NumCoordinateIndex; ++CoordinateIndex)
	{
		UVDensityAccs[CoordinateIndex].Reserve(NumTotalTriangles);
	}

	TArray<uint32> Indices;
	LODData.MultiSizeIndexContainer.GetIndexBuffer(Indices);
	if (!Indices.Num()) return;

	const uint32* SrcIndices = Indices.GetData() + Section.BaseIndex;
	uint32 NumTriangles = Section.NumTriangles;

	// Figure out Unreal unit per texel ratios.
	for (uint32 TriangleIndex = 0; TriangleIndex < NumTriangles; TriangleIndex++)
	{
		//retrieve indices
		uint32 Index0 = SrcIndices[TriangleIndex * 3];
		uint32 Index1 = SrcIndices[TriangleIndex * 3 + 1];
		uint32 Index2 = SrcIndices[TriangleIndex * 3 + 2];

		const float Area = FUVDensityAccumulator::GetTriangleArea(
			LODData.StaticVertexBuffers.PositionVertexBuffer.VertexPosition(Index0),
			LODData.StaticVertexBuffers.PositionVertexBuffer.VertexPosition(Index1),
			LODData.StaticVertexBuffers.PositionVertexBuffer.VertexPosition(Index2));

		if (Area > UE_SMALL_NUMBER)
		{
			for (int32 CoordinateIndex = 0; CoordinateIndex < NumCoordinateIndex; ++CoordinateIndex)
			{
				const float UVArea = FUVDensityAccumulator::GetUVChannelArea(
					LODData.StaticVertexBuffers.StaticMeshVertexBuffer.GetVertexUV(Index0, CoordinateIndex),
					LODData.StaticVertexBuffers.StaticMeshVertexBuffer.GetVertexUV(Index1, CoordinateIndex),
					LODData.StaticVertexBuffers.StaticMeshVertexBuffer.GetVertexUV(Index2, CoordinateIndex));

				UVDensityAccs[CoordinateIndex].PushTriangle(Area, UVArea);
			}
		}
	}

	for (int32 CoordinateIndex = 0; CoordinateIndex < NumCoordinateIndex; ++CoordinateIndex)
	{
		UVDensityAccs[CoordinateIndex].AccumulateDensity(OutWeightedUVDensities[CoordinateIndex], OutWeights[CoordinateIndex]);
	}
}
#endif

USkinnedAsset::USkinnedAsset(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{}

USkinnedAsset::~USkinnedAsset() 
{
#if WITH_EDITOR
	if (AsyncTask)
	{
		// Allow AsyncTask to finish if it hasn't yet, otherwise CheckIdle() could fail on deletion
		AsyncTask->EnsureCompletion();
	}
#endif
}

bool USkinnedAsset::IsValidMaterialIndex(int32 Index) const
{
	return GetMaterials().IsValidIndex(Index);
}

int32 USkinnedAsset::GetNumMaterials() const
{
	return GetMaterials().Num();
}

void USkinnedAsset::PostLoad()
{
	Super::PostLoad();

#if WITH_EDITOR
	FSkinnedAssetAsyncBuildScope AsyncBuildScope(this);
#endif

	FSkinnedAssetPostLoadContext Context;
	BeginPostLoadInternal(Context);

#if WITH_EDITOR
	if (FSkinnedAssetCompilingManager::Get().IsAsyncCompilationAllowed(this))
	{
		PrepareForAsyncCompilation();

		FQueuedThreadPool* ThreadPool = FSkinnedAssetCompilingManager::Get().GetThreadPool();
		EQueuedWorkPriority BasePriority = FSkinnedAssetCompilingManager::Get().GetBasePriority(this);

		AsyncTask = MakeUnique<FSkinnedAssetAsyncBuildTask>(this, MoveTemp(Context));
		if (!HasAnyDependenciesCompiling())
		{
			AsyncTask->StartBackgroundTask(ThreadPool, BasePriority, EQueuedWorkFlags::DoNotRunInsideBusyWait, GetRequiredMemoryForCompile(), TEXT("SkinnedAsset"));
		}
		
		if (HasCompileDependencies())
		{
			FSkinnedAssetCompilingManager::Get().AddSkinnedAssetsWithDependencies({ this });
		}
		else
		{
			FSkinnedAssetCompilingManager::Get().AddSkinnedAssets({ this });
		}
	}
	else
#endif
	{
		ExecutePostLoadInternal(Context);
		FinishPostLoadInternal(Context);
	}

	if (IsResourcePSOPrecachingEnabled() &&
		GetResourceForRendering() != nullptr)
	{
		// Assume GPU skinning when precaching PSOs
		bool bCPUSkin = false;
		ERHIFeatureLevel::Type FeatureLevel = GetWorld() ? GetWorld()->GetFeatureLevel() : GMaxRHIFeatureLevel;
		int32 MinLODIndex = GetMinLodIdx();
		
		FPSOPrecacheVertexFactoryDataPerMaterialIndexList VFsPerMaterials;
		GetVertexFactoryTypesPerMaterialIndex(VFsPerMaterials, nullptr, MinLODIndex, bCPUSkin, FeatureLevel);
		bool bAnySectionCastsShadows = GetResourceForRendering()->AnyRenderSectionCastsShadows(MinLODIndex);

		// Precache using default material & PSO precache params but mark movable
		const TArray<FSkeletalMaterial>& Materials = GetMaterials();
		FPSOPrecacheParams PrecachePSOParams;
		PrecachePSOParams.SetMobility(EComponentMobility::Movable);
		PrecachePSOParams.bCastShadow = bAnySectionCastsShadows;

		TArray<FMaterialPSOPrecacheRequestID> MaterialPSOPrecacheRequestIDs;
		for (auto VFsPerMaterial : VFsPerMaterials)
		{
			UMaterialInterface* MaterialInterface = Materials[VFsPerMaterial.MaterialIndex].MaterialInterface;
			if (MaterialInterface)
			{
				MaterialInterface->PrecachePSOs(VFsPerMaterial.VertexFactoryDataList, PrecachePSOParams, EPSOPrecachePriority::Medium, MaterialPSOPrecacheRequestIDs);
			}
		}
	}
}

void USkinnedAsset::GetVertexFactoryTypesPerMaterialIndex(FPSOPrecacheVertexFactoryDataPerMaterialIndexList& OutList,
	USkinnedMeshComponent* SkinnedMeshComponent, int32 MinLODIndex, bool bCPUSkin, ERHIFeatureLevel::Type FeatureLevel)
{
	FSkeletalMeshRenderData* SkelMeshRenderData = GetResourceForRendering();
	for (int32 LODIndex = MinLODIndex; LODIndex < SkelMeshRenderData->LODRenderData.Num(); LODIndex++)
	{
		FSkeletalMeshLODRenderData& LODRenderData = SkelMeshRenderData->LODRenderData[LODIndex];
		const FSkeletalMeshLODInfo& Info = *(GetLODInfo(LODIndex));

		// Check all render sections for the used material indices
		for (int32 SectionIndex = 0; SectionIndex < LODRenderData.RenderSections.Num(); SectionIndex++)
		{
			FSkelMeshRenderSection& RenderSection = LODRenderData.RenderSections[SectionIndex];

			// By default use the material index of the render section
			uint16 MaterialIndex = RenderSection.MaterialIndex;

			// Can be remapped by the LODInfo
			if (SectionIndex < Info.LODMaterialMap.Num() && IsValidMaterialIndex(Info.LODMaterialMap[SectionIndex]))
			{
				MaterialIndex = Info.LODMaterialMap[SectionIndex];
				MaterialIndex = FMath::Clamp(MaterialIndex, 0, GetNumMaterials());
			}

			FPSOPrecacheVertexFactoryDataPerMaterialIndex* VFsPerMaterial = OutList.FindByPredicate(
				[MaterialIndex](const FPSOPrecacheVertexFactoryDataPerMaterialIndex& Other) { return Other.MaterialIndex == MaterialIndex; });
			if (VFsPerMaterial == nullptr)
			{
				VFsPerMaterial = &OutList.AddDefaulted_GetRef();
				VFsPerMaterial->MaterialIndex = MaterialIndex;
			}

			// Find all the possible used vertex factory types needed to render this render section
			if (bCPUSkin)
			{
				// Force static from GPU point of view
				const FVertexFactoryType* CPUSkinVFType = &FLocalVertexFactory::StaticType;
				bool bSupportsManualVertexFetch = CPUSkinVFType->SupportsManualVertexFetch(GMaxRHIFeatureLevel);
				if (!bSupportsManualVertexFetch)
				{
					FVertexDeclarationElementList VertexElements;
					bool bOverrideColorVertexBuffer = false;
					FLocalVertexFactory::FDataType Data;
					LODRenderData.StaticVertexBuffers.InitComponentVF(nullptr /*VertexFactory*/, 0, bOverrideColorVertexBuffer, Data);
					FLocalVertexFactory::GetVertexElements(GMaxRHIFeatureLevel, EVertexInputStreamType::Default, bSupportsManualVertexFetch, Data, VertexElements);
					VFsPerMaterial->VertexFactoryDataList.AddUnique(FPSOPrecacheVertexFactoryData(CPUSkinVFType, VertexElements));
				}
				else
				{
					VFsPerMaterial->VertexFactoryDataList.AddUnique(FPSOPrecacheVertexFactoryData(CPUSkinVFType));
				}
			}
			else
			{
				// Add all the vertex factories which can be used for gpu skinning
				FSkeletalMeshObjectGPUSkin::GetUsedVertexFactoryData(SkelMeshRenderData, LODIndex, SkinnedMeshComponent, RenderSection, FeatureLevel, VFsPerMaterial->VertexFactoryDataList);
			}
		}
	}
}

#if WITH_EDITOR
bool USkinnedAsset::IsCompiling() const
{
	return AsyncTask != nullptr || AccessedProperties.load(std::memory_order_relaxed) != 0;
}

int64 USkinnedAsset::GetRequiredMemoryForCompile() const
{
	return 1024ull * 1024ull * CVarMemoryForSkinnedAssetCompile.GetValueOnAnyThread();
}

#endif // WITH_EDITOR

void USkinnedAsset::UpdateUVChannelData(bool bRebuildAll)
{
#if WITH_EDITORONLY_DATA
	// Once cooked, the data requires to compute the scales will not be CPU accessible.
	FSkeletalMeshRenderData* Resource = GetResourceForRendering();
	if (FPlatformProperties::HasEditorOnlyData() && Resource)
	{
		TArray<FSkeletalMaterial>& MeshMaterials = GetMaterials();
		for (int32 MaterialIndex = 0; MaterialIndex < MeshMaterials.Num(); ++MaterialIndex)
		{
			FMeshUVChannelInfo& UVChannelData = MeshMaterials[MaterialIndex].UVChannelData;

			// Skip it if we want to keep it.
			if (UVChannelData.IsInitialized() && (!bRebuildAll || UVChannelData.bOverrideDensities))
				continue;

			UpdateMaterialUVChannelData(UVChannelData, MaterialIndex);
		}

		Resource->SyncUVChannelData(GetMaterials());
	}
#endif
}

#if WITH_EDITORONLY_DATA

void USkinnedAsset::UpdateMaterialUVChannelData(FMeshUVChannelInfo& UVChannelData, int32 MaterialIndex)
{
	float WeightedUVDensities[TEXSTREAM_MAX_NUM_UVCHANNELS] = { 0, 0, 0, 0 };
	float Weights[TEXSTREAM_MAX_NUM_UVCHANNELS] = { 0, 0, 0, 0 };

	FSkeletalMeshRenderData* Resource = GetResourceForRendering();
	TArray<FSkeletalMaterial>& MeshMaterials = GetMaterials();
	for (int32 LODIndex = 0; LODIndex < Resource->LODRenderData.Num(); ++LODIndex)
	{
		const FSkeletalMeshLODRenderData& LODData = Resource->LODRenderData[LODIndex];
		const TArray<int32>& RemappedMaterialIndices = GetLODInfo(LODIndex)->LODMaterialMap;

		for (int32 SectionIndex = 0; SectionIndex < LODData.RenderSections.Num(); ++SectionIndex)
		{
			const FSkelMeshRenderSection& SectionInfo = LODData.RenderSections[SectionIndex];
			const int32 UsedMaterialIndex =
				SectionIndex < RemappedMaterialIndices.Num() && MeshMaterials.IsValidIndex(RemappedMaterialIndices[SectionIndex]) ?
				RemappedMaterialIndices[SectionIndex] :
				SectionInfo.MaterialIndex;

			if (UsedMaterialIndex != MaterialIndex)
				continue;

			AccumulateUVDensities(WeightedUVDensities, Weights, LODData, SectionInfo);
		}
	}

	UVChannelData.bInitialized = true;
	UVChannelData.bOverrideDensities = false;
	for (int32 CoordinateIndex = 0; CoordinateIndex < TEXSTREAM_MAX_NUM_UVCHANNELS; ++CoordinateIndex)
	{
		UVChannelData.LocalUVDensities[CoordinateIndex] = (Weights[CoordinateIndex] > UE_KINDA_SMALL_NUMBER) ? (WeightedUVDensities[CoordinateIndex] / Weights[CoordinateIndex]) : 0;
	}
}

#endif

void USkinnedAsset::AcquireAsyncProperty(uint64 AsyncProperties, ESkinnedAssetAsyncPropertyLockType LockType)
{
#if WITH_EDITOR
	if ((LockType & ESkinnedAssetAsyncPropertyLockType::ReadOnly) == ESkinnedAssetAsyncPropertyLockType::ReadOnly)
	{
		AccessedProperties |= AsyncProperties;
	}

	if ((LockType & ESkinnedAssetAsyncPropertyLockType::WriteOnly) == ESkinnedAssetAsyncPropertyLockType::WriteOnly)
	{
		ModifiedProperties |= AsyncProperties;
	}
#endif
}

void USkinnedAsset::ReleaseAsyncProperty(uint64 AsyncProperties, ESkinnedAssetAsyncPropertyLockType LockType)
{
#if WITH_EDITOR
	if ((LockType & ESkinnedAssetAsyncPropertyLockType::ReadOnly) == ESkinnedAssetAsyncPropertyLockType::ReadOnly)
	{
		AccessedProperties &= ~AsyncProperties;
	}

	if ((LockType & ESkinnedAssetAsyncPropertyLockType::WriteOnly) == ESkinnedAssetAsyncPropertyLockType::WriteOnly)
	{
		ModifiedProperties &= ~AsyncProperties;
	}
#endif
}

void USkinnedAsset::WaitUntilAsyncPropertyReleasedInternal(uint64 AsyncProperties, ESkinnedAssetAsyncPropertyLockType LockType) const
{
#if WITH_EDITOR
	// We need to protect internal skinned asset data from race conditions during async build
	if (IsCompiling())
	{
		if (FSkinnedAssetAsyncBuildScope::ShouldWaitOnLockedProperties(this))
		{
			bool bIsLocked = true;
			// We can remove the lock if we're accessing in read-only and there is no write-lock
			if ((LockType & ESkinnedAssetAsyncPropertyLockType::ReadOnly) == ESkinnedAssetAsyncPropertyLockType::ReadOnly)
			{
				// Maintain the lock if the write-lock bit is non-zero
				bIsLocked &= (ModifiedProperties & AsyncProperties) != 0;
			}

			if (bIsLocked)
			{
				FString PropertyName = GetAsyncPropertyName(AsyncProperties);
				TRACE_CPUPROFILER_EVENT_SCOPE_TEXT(*FString::Printf(TEXT("SkinnedAssetCompilationStall %s"), *PropertyName));

				if (IsInGameThread())
				{
					UE_LOGF(
						LogSkinnedAsset,
						Verbose,
						"Accessing property %ls of the SkinnedAsset while it is still being built asynchronously will force it to be compiled before continuing. "
							"For better performance, consider making the caller async aware so it can wait until the static mesh is ready to access this property."
							"To better understand where those calls are coming from, you can use Editor.AsyncAssetDumpStallStacks on the console.",
						*PropertyName
					);

					FSkinnedAssetCompilingManager::Get().FinishCompilation({ const_cast<USkinnedAsset*>(this) });
				}
				else
				{
					// Trying to access a property from another thread that cannot force finish the compilation is invalid
					ensureMsgf(
						false,
						TEXT("Accessing property %s of the SkinnedAsset while it is still being built asynchronously is only supported on the game-thread. "
							"To avoid any race-condition, consider finishing the compilation before pushing tasks to other threads or making higher-level game-thread code async aware so it "
							"schedules the task only when the static mesh's compilation is finished. If this is a blocker, you can disable async static mesh from the editor experimental settings."),
						*PropertyName
					);
				}
			}
		}
		// If we're accessing this property from the async build thread, make sure the property is still protected from access from other threads.
		else
		{
			bool bIsLocked = true;
			if ((LockType & ESkinnedAssetAsyncPropertyLockType::ReadOnly) == ESkinnedAssetAsyncPropertyLockType::ReadOnly)
			{
				bIsLocked &= (AccessedProperties & AsyncProperties) != 0;
			}

			if ((LockType & ESkinnedAssetAsyncPropertyLockType::WriteOnly) == ESkinnedAssetAsyncPropertyLockType::WriteOnly)
			{
				bIsLocked &= (ModifiedProperties & AsyncProperties) != 0;
			}
			ensureMsgf(bIsLocked, TEXT("Property %s has not been locked properly for use by async build"), *GetAsyncPropertyName(AsyncProperties));
		}
	}
#endif
}

extern bool bAnim_SkeletalMesh_ISPC_Enabled;
static bool IsISPCEnabled()
{
	// By default when ANIM_SKELETAL_MESH_ISPC_ENABLED_DEFAULT and CVarAnimSkeletalMeshISPCEnabled are not set, bAnim_SkeletalMesh_ISPC_Enabled should be true,
	// so when it is false it means one of the two settings are changed. To be backwards compatible, its overridden value takes priority.
	if (!bAnim_SkeletalMesh_ISPC_Enabled)
	{
#if INTEL_ISPC
		static bool bWarningLogged = false;
		if (!bWarningLogged)
		{
			UE_LOGF(
				LogSkinnedAsset,
				Warning,
				"ANIM_SKELETAL_MESH_ISPC_ENABLED_DEFAULT and CVar 'a.SkeletalMesh.ISPC' are deprecated and will be removed in future releases."
					"Please switch to ANIM_SKINNED_ASSET_ISPC_ENABLED_DEFAULT and 'a.SkinnedAsset.ISPC'"
			);
			bWarningLogged = true;
		}
#endif
		return false;
	}

	// No old values detected, use the new setting
	return bAnim_SkinnedAsset_ISPC_Enabled;
}

ESkeletalMeshVertexFlags USkinnedAsset::GetVertexBufferFlags() const
{
	return GetHasVertexColors() ? ESkeletalMeshVertexFlags::HasVertexColors : ESkeletalMeshVertexFlags::None;
}

void USkinnedAsset::FillComponentSpaceTransforms(const TArray<FTransform>& InBoneSpaceTransforms,
												 const TArray<FBoneIndexType>& InFillComponentSpaceTransformsRequiredBones, 
												 TArray<FTransform>& OutComponentSpaceTransforms) const
{
	ANIM_MT_SCOPE_CYCLE_COUNTER(FillComponentSpaceTransforms, !IsInGameThread());

	// right now all this does is populate DestSpaceBases
	checkf(GetRefSkeleton().GetNum() == InBoneSpaceTransforms.Num(),
		TEXT("RefSkeleton bone count (%d) and input bone transforms array count (%d) mismatches. Skinned Asset: [%s]"), GetRefSkeleton().GetNum(), InBoneSpaceTransforms.Num(), *GetPathName());
	checkf(GetRefSkeleton().GetNum() == OutComponentSpaceTransforms.Num(),
		TEXT("RefSkeleton bone count (%d) and output bone transforms array count (%d) mismatches. Skinned Asset: [%s]"), GetRefSkeleton().GetNum(), OutComponentSpaceTransforms.Num(), *GetPathName());

	const int32 NumBones = InBoneSpaceTransforms.Num();
	
	if (!NumBones)
	{
		return;
	}

#if DO_GUARD_SLOW
	/** Keep track of which bones have been processed for fast look up */
	TArray<uint8, TInlineAllocator<256>> BoneProcessed;
	BoneProcessed.AddZeroed(NumBones);
#endif

	const FTransform* LocalTransformsData = InBoneSpaceTransforms.GetData();
	FTransform* ComponentSpaceData = OutComponentSpaceTransforms.GetData();

	// First bone (if we have one) is always root bone, and it doesn't have a parent.
	{
		check(InFillComponentSpaceTransformsRequiredBones.Num() == 0 || InFillComponentSpaceTransformsRequiredBones[0] == 0);
		OutComponentSpaceTransforms[0] = InBoneSpaceTransforms[0];

#if DO_GUARD_SLOW
		// Mark bone as processed
		BoneProcessed[0] = 1;
#endif
	}

	if (IsISPCEnabled())
	{
#if INTEL_ISPC
		// Disable validation as open writes can cause false-positives with
		// the AutoRTFM sanitizer.
		UE_AUTORTFM_OPEN_NO_SANITIZE
		{
			AutoRTFM::RecordOpenWrite(
				ComponentSpaceData,
				OutComponentSpaceTransforms.Num() * sizeof(FTransform));
			ispc::FillComponentSpaceTransforms(
				reinterpret_cast<ispc::FTransform*>(ComponentSpaceData),
				reinterpret_cast<const ispc::FTransform*>(LocalTransformsData),
				InFillComponentSpaceTransformsRequiredBones.GetData(),
				reinterpret_cast<const uint8*>(GetRefSkeleton().GetRefBoneInfo().GetData()),
				sizeof(FMeshBoneInfo),
				offsetof(FMeshBoneInfo, ParentIndex),
				InFillComponentSpaceTransformsRequiredBones.Num());
		};
#endif
	}
	else
	{
		for (int32 i = 1; i < InFillComponentSpaceTransformsRequiredBones.Num(); i++)
		{
			const int32 BoneIndex = InFillComponentSpaceTransformsRequiredBones[i];
			FTransform* SpaceBase = ComponentSpaceData + BoneIndex;

			FPlatformMisc::Prefetch(SpaceBase);

#if DO_GUARD_SLOW
			// Mark bone as processed
			BoneProcessed[BoneIndex] = 1;
#endif
			// For all bones below the root, final component-space transform is relative transform * component-space transform of parent.
			const int32 ParentIndex = GetRefSkeleton().GetParentIndex(BoneIndex);
			FTransform* ParentSpaceBase = ComponentSpaceData + ParentIndex;
			FPlatformMisc::Prefetch(ParentSpaceBase);

#if DO_GUARD_SLOW
			// Check the precondition that Parents occur before Children in the RequiredBones array.
			checkSlow(BoneProcessed[ParentIndex] == 1);
#endif
			FTransform::Multiply(SpaceBase, LocalTransformsData + BoneIndex, ParentSpaceBase);

			SpaceBase->NormalizeRotation();

			checkSlow(SpaceBase->IsRotationNormalized());
			checkSlow(!SpaceBase->ContainsNaN());
		}
	}
}

TArray<FSkeletalMeshLODInfo>& USkinnedAsset::GetMeshLodInfoDummyArray()
{
	static TArray<FSkeletalMeshLODInfo> Dummy;
	return Dummy;
}

TArray<FSkeletalMaterial>& USkinnedAsset::GetSkeletalMaterialDummyArray()
{
	static TArray<FSkeletalMaterial> Dummy;
	return Dummy;
}

#if WITH_EDITOR
bool USkinnedAsset::TryCancelAsyncTasks()
{
	if (AsyncTask)
	{
		if (AsyncTask->IsDone() || AsyncTask->Cancel())
		{
			AsyncTask.Reset();
		}
	}

	return AsyncTask == nullptr;
}
#endif //WITH_EDITOR

FString USkinnedAsset::GetLODPathName(const USkinnedAsset* Mesh, int32 LODIndex)
{
#if RHI_ENABLE_RESOURCE_INFO
	return FString::Printf(TEXT("%s [LOD%d]"), Mesh ? *Mesh->GetPathName() : TEXT("UnknownSkinnedAsset"), LODIndex);
#else
	return TEXT("");
#endif
}

FSkinnedMeshComponentRecreateRenderStateContext::FSkinnedMeshComponentRecreateRenderStateContext(USkinnedAsset* InSkinnedAsset, bool InRefreshBounds /*= false*/)
	: bRefreshBounds(InRefreshBounds)
{
	TSet<FSceneInterface*> Scenes;

	FObjectCacheContextScope ObjectCacheScope;
	TObjectCacheIterator<ISkinnedMeshComponent> SkinnedMeshComponentsIter = InSkinnedAsset
		? ObjectCacheScope.GetContext().GetSkinnedMeshComponents(InSkinnedAsset)
		: ObjectCacheScope.GetContext().GetSkinnedMeshComponents();

	for (ISkinnedMeshComponent* Component : SkinnedMeshComponentsIter)
	{
		IPrimitiveComponent* PrimitiveComponent = Component->GetPrimitiveComponentInterface();

		if (PrimitiveComponent->IsRenderStateCreated())
		{
			check(PrimitiveComponent->IsRegistered());
			PrimitiveComponent->DestroyRenderState();
			SkinnedMeshComponents.Add({PrimitiveComponent->GetUObject(), Component});
			Scenes.Add(PrimitiveComponent->GetScene());
		}
	}

	if (Scenes.Num())
	{
		UpdateAllPrimitiveSceneInfosForScenes(MoveTemp(Scenes));

		// Flush the rendering commands generated by the detachments.
		// The skeletal mesh scene proxies reference the USkinnedAsset, and this ensures that they are cleaned up before the USkinnedAsset changes.
		FlushRenderingCommands();
	}
}

FSkinnedMeshComponentRecreateRenderStateContext::~FSkinnedMeshComponentRecreateRenderStateContext()
{
	TSet<FSceneInterface*> Scenes;

	for (const TPair<TWeakObjectPtr<>, ISkinnedMeshComponent*>& ComponentPair : SkinnedMeshComponents)
	{
		// Check if the object is stale and early out if so
		if (ComponentPair.Key.Get() == nullptr)
		{
			continue;
		}

		IPrimitiveComponent* PrimitiveComponent = ComponentPair.Value->GetPrimitiveComponentInterface();

		// TODO: Perhaps move UpdateBounds calcs into IPrimitiveComponent, or introduce ISceneComponent?
		if (USceneComponent* SceneComponent = Cast<USceneComponent>(PrimitiveComponent->GetUObject()))
		{
			if (bRefreshBounds)
			{
				SceneComponent->UpdateBounds();
			}
		}

		if (PrimitiveComponent->IsRegistered() && !PrimitiveComponent->IsRenderStateCreated() && PrimitiveComponent->ShouldCreateRenderState())
		{
			PrimitiveComponent->CreateRenderState(nullptr);
			Scenes.Add(PrimitiveComponent->GetScene());
		}
	}

	UpdateAllPrimitiveSceneInfosForScenes(MoveTemp(Scenes));
}

