// Copyright Epic Games, Inc. All Rights Reserved.

#define EXPLICIT_VECTOR4 1
#define EXPLICIT_MATRIX 1

#include "Math/Vector.isph"
#include "Math/Matrix.isph"
#include "Math/Quat.isph"
#include "Math/Transform.isph"

export void FillComponentSpaceTransforms(uniform FTransform ComponentSpaceData[],
										const uniform FTransform LocalTransformsData[],
										const uniform unsigned int16 FillComponentSpaceTransformsRequiredBones[],
										const uniform unsigned int8 * uniform ParentBones,
										const uniform int ParentBoneSize,
										const uniform int ParentIndexOffset,
										const uniform int NumBones)
{
	for (uniform int i = 1; i < NumBones; i++)
	{
		const uniform int BoneIndex = FillComponentSpaceTransformsRequiredBones[i];
		uniform FTransform SpaceBase = ComponentSpaceData[BoneIndex];

		// For all bones below the root, final component-space transform is relative transform * component-space transform of parent.
		const uniform unsigned int8 * uniform ParentIndexPtr = ParentBones + ((ParentBoneSize * BoneIndex) + ParentIndexOffset);
		const uniform int ParentIndex = *((uniform int * uniform)ParentIndexPtr);
		const uniform FTransform ParentSpaceBase = ComponentSpaceData[ParentIndex];

		const uniform FTransform LocalTransform = LocalTransformsData[BoneIndex];

		TransformMultiply(SpaceBase, LocalTransform, ParentSpaceBase);
		SpaceBase.Rotation = VectorNormalizeQuaternion(SpaceBase.Rotation);

		ComponentSpaceData[BoneIndex] = SpaceBase;
	}
}
