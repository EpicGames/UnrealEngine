// Copyright Epic Games, Inc. All Rights Reserved.

#include "Engine/SkinnedAssetAsyncCompileUtils.h"
#include "Engine/SkinnedAsset.h"

#if WITH_EDITOR

/*-----------------------------------------------------------------------------
	FSkinnedAssetAsyncBuildScope
-----------------------------------------------------------------------------*/
thread_local const USkinnedAsset* FSkinnedAssetAsyncBuildScope::SkinnedAssetBeingAsyncCompiled = nullptr;

FSkinnedAssetAsyncBuildScope::FSkinnedAssetAsyncBuildScope(const USkinnedAsset* SkinnedAsset)
{
	PreviousScope = SkinnedAssetBeingAsyncCompiled;
	SkinnedAssetBeingAsyncCompiled = SkinnedAsset;
}

FSkinnedAssetAsyncBuildScope::~FSkinnedAssetAsyncBuildScope()
{
	check(SkinnedAssetBeingAsyncCompiled);
	SkinnedAssetBeingAsyncCompiled = PreviousScope;
}

bool FSkinnedAssetAsyncBuildScope::ShouldWaitOnLockedProperties(const USkinnedAsset* SkinnedAsset)
{
	return SkinnedAssetBeingAsyncCompiled != SkinnedAsset;
}


/*-----------------------------------------------------------------------------
	FSkinnedAssetAsyncBuildWorker
-----------------------------------------------------------------------------*/
void FSkinnedAssetAsyncBuildWorker::DoWork()
{
	if (PostLoadContext.IsSet())
	{
		SkinnedAsset->ExecutePostLoadInternal(*PostLoadContext);
	}

	if (BuildContext.IsSet())
	{
		SkinnedAsset->ExecuteBuildInternal(*BuildContext);
	}

	if (AsyncTaskContext.IsSet())
	{
		SkinnedAsset->ExecuteAsyncTaskInternal(*AsyncTaskContext);
	}
}

#endif //WITH_EDITOR
