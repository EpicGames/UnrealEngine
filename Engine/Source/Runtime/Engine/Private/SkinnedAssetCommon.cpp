// Copyright Epic Games, Inc. All Rights Reserved.

#include "Engine/SkinnedAssetCommon.h"

#include "Animation/AnimSequence.h"
#include "Engine/SkeletalMeshLODSettings.h"
#include "Engine/TextureStreamingTypes.h"
#include "HAL/FileManager.h"
#include "Interfaces/ITargetPlatform.h"
#include "Materials/MaterialInterface.h"
#include "Misc/Paths.h"
#include "Misc/SecureHash.h"
#include "Rendering/SkeletalMeshLODModel.h"
#include "SkeletalMeshLegacyCustomVersions.h"
#include "UObject/CoreObjectVersion.h"
#include "UObject/EditorObjectVersion.h"
#include "UObject/FortniteMainBranchObjectVersion.h"
#include "UObject/FortniteSeasonBranchObjectVersion.h"
#include "UObject/RenderingObjectVersion.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SkinnedAssetCommon)


DEFINE_STAT(STAT_ClothTotalTime);
DEFINE_STAT(STAT_ClothWriteback);

/*-----------------------------------------------------------------------------
	FSkeletalMeshLODInfo
-----------------------------------------------------------------------------*/

//Init static string
FString FMorphTargetImportedSourceFileInfo::GeneratedByEnginePrefix = TEXT("GeneratedByEngine_");

const FString& FMorphTargetImportedSourceFileInfo::GetSourceFilename() const
{
	static FString EmptyString = FString();
	if(!IsGeneratedByEngine())
	{
		return SourceFilename;
	}
	return EmptyString;
}

bool FMorphTargetImportedSourceFileInfo::IsGeneratedByEngine() const
{
	return SourceFilename.StartsWith(GeneratedByEnginePrefix);
}

void FMorphTargetImportedSourceFileInfo::SetGeneratedByEngine(bool bInGeneratedByEngine)
{
	FString NewSourceFilename = GeneratedByEnginePrefix;
	NewSourceFilename += FGuid::NewGuid().ToString(EGuidFormats::Base36Encoded);
	SetSourceFilename(NewSourceFilename);
}

void FMorphTargetImportedSourceFileInfo::SetSourceFilename(const FString& Filename)
{
	FString DerivedDataKey = Filename;
	if (FPaths::FileExists(Filename))
	{
		DerivedDataKey += IFileManager::Get().GetTimeStamp(*Filename).ToString();
	}

	const auto CharArray = DerivedDataKey.GetCharArray();
	FSHA1 Sha;
	Sha.Update(reinterpret_cast<const uint8*>(CharArray.GetData()), CharArray.Num() * CharArray.GetTypeSize());
	Sha.Final();
	// Retrieve the hash and use it to construct a pseudo-GUID.
	uint32 Hash[5];
	Sha.GetHash((uint8*)Hash);
	DerivedDataHash = FGuid(Hash[0] ^ Hash[4], Hash[1], Hash[2], Hash[3]);

	SourceFilename = Filename;
}

FArchive& operator<<(FArchive& Ar, FMorphTargetImportedSourceFileInfo& MorphTargetImportedSourceFileInfo)
{
	Ar << MorphTargetImportedSourceFileInfo.SourceFilename;
	Ar << MorphTargetImportedSourceFileInfo.DerivedDataHash;
	return Ar;
}

#if WITH_EDITORONLY_DATA

static void SerializeReductionSettingsForDDC(FArchive& Ar, FSkeletalMeshOptimizationSettings& ReductionSettings)
{
	check(Ar.IsSaving());
	// Note: this serializer is only used to build the mesh DDC key, no versioning is required
	FArchive_Serialize_BitfieldBool(Ar, ReductionSettings.bRemapMorphTargets);
	FArchive_Serialize_BitfieldBool(Ar, ReductionSettings.bRecalcNormals);
	FArchive_Serialize_BitfieldBool(Ar, ReductionSettings.bEnforceBoneBoundaries);
	FArchive_Serialize_BitfieldBool(Ar, ReductionSettings.bMergeCoincidentVertBones);
	FArchive_Serialize_BitfieldBool(Ar, ReductionSettings.bLockEdges);
	FArchive_Serialize_BitfieldBool(Ar, ReductionSettings.bLockColorBounaries);
	FArchive_Serialize_BitfieldBool(Ar, ReductionSettings.bImproveTrianglesForCloth);
	Ar << ReductionSettings.TerminationCriterion;
	Ar << ReductionSettings.NumOfTrianglesPercentage;
	Ar << ReductionSettings.NumOfVertPercentage;
	Ar << ReductionSettings.MaxNumOfTriangles;
	Ar << ReductionSettings.MaxNumOfVerts;

	// Keep old DDC keys if these are not set
	if (ReductionSettings.MaxNumOfTrianglesPercentage != MAX_uint32 ||
		ReductionSettings.MaxNumOfVertsPercentage != MAX_uint32)
	{
		uint32 AvoidCachePoisoningFromOldBug = 0;
		Ar << AvoidCachePoisoningFromOldBug;
		Ar << ReductionSettings.MaxNumOfTrianglesPercentage;
		Ar << ReductionSettings.MaxNumOfVertsPercentage;
	}

	Ar << ReductionSettings.MaxDeviationPercentage;
	Ar << ReductionSettings.ReductionMethod;
	Ar << ReductionSettings.SilhouetteImportance;
	Ar << ReductionSettings.TextureImportance;
	Ar << ReductionSettings.ShadingImportance;
	Ar << ReductionSettings.SkinningImportance;
	Ar << ReductionSettings.WeldingThreshold;
	Ar << ReductionSettings.NormalsThreshold;
	Ar << ReductionSettings.MaxBonesPerVertex;
	Ar << ReductionSettings.VolumeImportance;
	Ar << ReductionSettings.BaseLOD;
}

static void SerializeBuildSettingsForDDC(FArchive& Ar, FSkeletalMeshBuildSettings& BuildSettings)
{
	check(Ar.IsSaving());
	// Note: this serializer is only used to build the mesh DDC key, no versioning is required
	FArchive_Serialize_BitfieldBool(Ar, BuildSettings.bRecomputeNormals);
	FArchive_Serialize_BitfieldBool(Ar, BuildSettings.bRecomputeTangents);
	FArchive_Serialize_BitfieldBool(Ar, BuildSettings.bUseMikkTSpace);
	FArchive_Serialize_BitfieldBool(Ar, BuildSettings.bComputeWeightedNormals);
	FArchive_Serialize_BitfieldBool(Ar, BuildSettings.bRemoveDegenerates);
	FArchive_Serialize_BitfieldBool(Ar, BuildSettings.bUseFullPrecisionUVs);
	FArchive_Serialize_BitfieldBool(Ar, BuildSettings.bUseBackwardsCompatibleF16TruncUVs);
	FArchive_Serialize_BitfieldBool(Ar, BuildSettings.bUseHighPrecisionTangentBasis);
	FArchive_Serialize_BitfieldBool(Ar, BuildSettings.bUseHighPrecisionSkinWeights);
	FArchive_Serialize_BitfieldBool(Ar, BuildSettings.bOptimizeForInstancing);
	Ar << BuildSettings.ThresholdPosition;
	Ar << BuildSettings.ThresholdTangentNormal;
	Ar << BuildSettings.ThresholdUV;
	Ar << BuildSettings.MorphThresholdPosition;
	Ar << BuildSettings.BoneInfluenceLimit;
}

FGuid FSkeletalMeshLODInfo::ComputeDeriveDataCacheKey(const FSkeletalMeshLODGroupSettings* SkeletalMeshLODGroupSettings)
{
	// Serialize the LOD info members, the BuildSettings and the ReductionSettings into a temporary array.
	TArray<uint8> TempBytes;
	TempBytes.Reserve(64);
	//The archive is flagged as persistent so that machines of different endianness produce identical binary results.
	FMemoryWriter Ar(TempBytes, /*bIsPersistent=*/ true);

	Ar << BonesToRemove;
	Ar << BonesToPrioritize;
	Ar << SectionsToPrioritize;
	Ar << WeightOfPrioritization;
	Ar << MorphTargetPositionErrorTolerance;
	Ar << ImportedMorphTargetSourceFilename;

	//TODO: Ask the derivedata key of the UObject reference by FSoftObjectPath. So if someone change the UObject, this LODs will get dirty
	//and will be rebuild.
	if (BakePose != nullptr)
	{
		FString BakePosePath = BakePose->GetFullName();
		Ar << BakePosePath;
	}
	if (BakePoseOverride != nullptr)
	{
		FString BakePoseOverridePath = BakePoseOverride->GetFullName();
		Ar << BakePoseOverridePath;
	}
	
	FArchive_Serialize_BitfieldBool(Ar, bAllowCPUAccess);
	FArchive_Serialize_BitfieldBool(Ar, bBuildHalfEdgeBuffers);
	FArchive_Serialize_BitfieldBool(Ar, bSupportUniformlyDistributedSampling);

	//Use the LOD settings asset if there is one
	FSkeletalMeshOptimizationSettings RealReductionSettings = ReductionSettings;
	if (SkeletalMeshLODGroupSettings != nullptr)
	{
		RealReductionSettings = SkeletalMeshLODGroupSettings->GetReductionSettings();
	}

	SerializeBuildSettingsForDDC(Ar, BuildSettings);
	SerializeReductionSettingsForDDC(Ar, RealReductionSettings);

	int32 NumRenderableAttributes = 0;
	for (const FSkeletalMeshVertexAttributeInfo& VertexAttributeInfo: VertexAttributes)
	{
		if (VertexAttributeInfo.IsEnabledForRender())
		{
			FString AttributeName(VertexAttributeInfo.Name.ToString());
			Ar << AttributeName;
			Ar << VertexAttributeInfo.DataType;
			NumRenderableAttributes++;
		}
	}
	Ar << NumRenderableAttributes;
	

	FSHA1 Sha;
	Sha.Update(TempBytes.GetData(), TempBytes.Num() * TempBytes.GetTypeSize());
	Sha.Final();
	// Retrieve the hash and use it to construct a pseudo-GUID.
	uint32 Hash[5];
	Sha.GetHash((uint8*)Hash);
	FGuid Guid = FGuid(Hash[0] ^ Hash[4], Hash[1], Hash[2], Hash[3]);
	return Guid;
}
#endif //WITH_EDITORONLY_DATA


/*-----------------------------------------------------------------------------
	FSectionReference
-----------------------------------------------------------------------------*/
#if WITH_EDITOR
bool FSectionReference::IsValidToEvaluate(const FSkeletalMeshLODModel& LodModel) const
{
	return GetMeshLodSection(LodModel) != nullptr;
}

const FSkelMeshSection* FSectionReference::GetMeshLodSection(const FSkeletalMeshLODModel& LodModel) const
{
	for (int32 LodModelSectionIndex = 0; LodModelSectionIndex < LodModel.Sections.Num(); ++LodModelSectionIndex)
	{
		const FSkelMeshSection& Section = LodModel.Sections[LodModelSectionIndex];
		if (Section.ChunkedParentSectionIndex == INDEX_NONE && SectionIndex == Section.OriginalDataSectionIndex)
		{
			return &Section;
		}
	}
	return nullptr;
}

int32 FSectionReference::GetMeshLodSectionIndex(const FSkeletalMeshLODModel& LodModel) const
{
	for (int32 LodModelSectionIndex = 0; LodModelSectionIndex < LodModel.Sections.Num(); ++LodModelSectionIndex)
	{
		const FSkelMeshSection& Section = LodModel.Sections[LodModelSectionIndex];
		if (Section.ChunkedParentSectionIndex == INDEX_NONE && SectionIndex == Section.OriginalDataSectionIndex)
		{
			return LodModelSectionIndex;
		}
	}
	return INDEX_NONE;
}
#endif // #if WITH_EDITOR

/*-----------------------------------------------------------------------------
	FSkeletalMaterial
-----------------------------------------------------------------------------*/

bool operator== (const FSkeletalMaterial& LHS, const FSkeletalMaterial& RHS)
{
	return (LHS.MaterialInterface == RHS.MaterialInterface);
}

bool operator== (const FSkeletalMaterial& LHS, const UMaterialInterface& RHS)
{
	return (LHS.MaterialInterface == &RHS);
}

bool operator== (const UMaterialInterface& LHS, const FSkeletalMaterial& RHS)
{
	return (RHS.MaterialInterface == &LHS);
}

FArchive& operator<<(FArchive& Ar, FMeshUVChannelInfo& ChannelData)
{
	Ar << ChannelData.bInitialized;
	Ar << ChannelData.bOverrideDensities;

	for (int32 CoordIndex = 0; CoordIndex < TEXSTREAM_MAX_NUM_UVCHANNELS; ++CoordIndex)
	{
		Ar << ChannelData.LocalUVDensities[CoordIndex];
	}

	return Ar;
}

#if WITH_EDITORONLY_DATA
void FSkeletalMaterial::DeclareCustomVersions(FArchive& Ar)
{
	Ar.UsingCustomVersion(FEditorObjectVersion::GUID);
	Ar.UsingCustomVersion(FCoreObjectVersion::GUID);
	Ar.UsingCustomVersion(FFortniteMainBranchObjectVersion::GUID);
}
#endif

FArchive& operator<<(FArchive& Ar, FSkeletalMaterial& Elem)
{
	Ar.UsingCustomVersion(FEditorObjectVersion::GUID);
	Ar.UsingCustomVersion(FCoreObjectVersion::GUID);
	Ar.UsingCustomVersion(FFortniteMainBranchObjectVersion::GUID);

	Ar << Elem.MaterialInterface;

	//Use the automatic serialization instead of this custom operator
	if (Ar.CustomVer(FEditorObjectVersion::GUID) >= FEditorObjectVersion::RefactorMeshEditorMaterials)
	{
		Ar << Elem.MaterialSlotName;

		bool bSerializeImportedMaterialSlotName = !Ar.IsCooking() || Ar.CookingTarget()->HasEditorOnlyData();
		if (Ar.CustomVer(FCoreObjectVersion::GUID) >= FCoreObjectVersion::SkeletalMaterialEditorDataStripping)
		{
			Ar << bSerializeImportedMaterialSlotName;
		}
		else if (!FPlatformProperties::HasEditorOnlyData())
		{
			bSerializeImportedMaterialSlotName = false;
		}
		if (bSerializeImportedMaterialSlotName)
		{
#if WITH_EDITORONLY_DATA
			Ar << Elem.ImportedMaterialSlotName;
#else
			FName UnusedImportedMaterialSlotName;
			Ar << UnusedImportedMaterialSlotName;
#endif
		}
	}
#if WITH_EDITORONLY_DATA
	else
	{
		if (Ar.UEVer() >= VER_UE4_MOVE_SKELETALMESH_SHADOWCASTING)
		{
			Ar << Elem.bEnableShadowCasting_DEPRECATED;
		}

		Ar.UsingCustomVersion(FRecomputeTangentCustomVersion::GUID);
		if (Ar.CustomVer(FRecomputeTangentCustomVersion::GUID) >= FRecomputeTangentCustomVersion::RuntimeRecomputeTangent)
		{
			Ar << Elem.bRecomputeTangent_DEPRECATED;
		}
	}
#endif

	if (!Ar.IsLoading() || Ar.CustomVer(FRenderingObjectVersion::GUID) >= FRenderingObjectVersion::TextureStreamingMeshUVChannelData)
	{
		Ar << Elem.UVChannelData;
	}

	if (!Ar.IsLoading() || Ar.CustomVer(FFortniteMainBranchObjectVersion::GUID) >= FFortniteMainBranchObjectVersion::MeshMaterialSlotOverlayMaterialAdded)
	{
		Ar << Elem.OverlayMaterialInterface;
	}
	else
	{
		Elem.OverlayMaterialInterface = nullptr;
	}
	return Ar;
}
