// Copyright Epic Games, Inc. All Rights Reserved.

#include "SkinnedAssetCompiler.h"
#include "Components/SkinnedMeshComponent.h"
#include "Engine/SkinnedAsset.h"
#include "UObject/UnrealType.h"

#if WITH_EDITOR

#include "AssetCompilingManager.h"
#include "AsyncCompilationHelpers.h"
#include "EngineLogs.h"
#include "ObjectCacheContext.h"
#include "Settings/EditorExperimentalSettings.h"
#include "Misc/QueuedThreadPoolWrapper.h"
#include "Engine/SkinnedAssetAsyncCompileUtils.h"
#include "UObject/StrongObjectPtr.h"
#include "ShaderCompiler.h"
#include "TextureCompiler.h"
#include "ProfilingDebugging/CountersTrace.h"
#include "Algo/NoneOf.h"

#define LOCTEXT_NAMESPACE "SkinnedAssetCompiler"

static AsyncCompilationHelpers::FAsyncCompilationStandardCVars CVarAsyncSkinnedAssetStandard(
	TEXT("SkinnedAsset"),
	TEXT("skinned assets"),
	FConsoleCommandDelegate::CreateLambda(
		[]()
		{
			FSkinnedAssetCompilingManager::Get().FinishAllCompilation();
		}
	));

namespace SkinnedAssetCompilingManagerImpl
{
	static void EnsureInitializedCVars()
	{
		static bool bIsInitialized = false;

		if (!bIsInitialized)
		{
			bIsInitialized = true;
			
			AsyncCompilationHelpers::EnsureInitializedCVars(
				TEXT("skinnedasset"),
				CVarAsyncSkinnedAssetStandard.AsyncCompilation,
				CVarAsyncSkinnedAssetStandard.AsyncCompilationMaxConcurrency,
				GET_MEMBER_NAME_CHECKED(UEditorExperimentalSettings, bEnableAsyncSkinnedAssetCompilation));
		}
	}
}

FSkinnedAssetCompilingManager::FSkinnedAssetCompilingManager()
	: Notification(MakeUnique<FAsyncCompilationNotification>(GetAssetNameFormat()))
{
	SkinnedAssetCompilingManagerImpl::EnsureInitializedCVars();
	PostReachabilityAnalysisHandle = FCoreUObjectDelegates::PostReachabilityAnalysis.AddRaw(this, &FSkinnedAssetCompilingManager::OnPostReachabilityAnalysis);
	PreGarbageCollectHandle = FCoreUObjectDelegates::GetPreGarbageCollectDelegate().AddRaw(this, &FSkinnedAssetCompilingManager::OnPreGarbageCollect);
}

FName FSkinnedAssetCompilingManager::GetStaticAssetTypeName()
{
	return TEXT("UE-SkinnedAsset");
}

FName FSkinnedAssetCompilingManager::GetAssetTypeName() const
{
	return GetStaticAssetTypeName();
}

FTextFormat FSkinnedAssetCompilingManager::GetAssetNameFormat() const
{
	return LOCTEXT("SkinnedAssetNameFormat", "{0}|plural(one=Skinned Asset,other=Skinned Assets)");
}

TArrayView<FName> FSkinnedAssetCompilingManager::GetDependentTypeNames() const
{
	// Texture and shaders can affect materials which can affect Skinned Assets once they are visible.
	// Adding these dependencies can reduces the actual number of render state update we need to do in a frame
	static FName DependentTypeNames[] = 
	{
		FTextureCompilingManager::GetStaticAssetTypeName(), 
		FShaderCompilingManager::GetStaticAssetTypeName() 
	};
	return TArrayView<FName>(DependentTypeNames);
}

int32 FSkinnedAssetCompilingManager::GetNumRemainingAssets() const
{
	return GetNumRemainingJobs();
}

EQueuedWorkPriority FSkinnedAssetCompilingManager::GetBasePriority(USkinnedAsset* InSkinnedAsset) const
{
	return EQueuedWorkPriority::Low;
}

FQueuedThreadPool* FSkinnedAssetCompilingManager::GetThreadPool() const
{
	static FQueuedThreadPoolDynamicWrapper* GSkinnedAssetThreadPool = nullptr;
	if (GSkinnedAssetThreadPool == nullptr && FAssetCompilingManager::Get().GetThreadPool() != nullptr)
	{
		// For now, skinned assets have almost no high-level awareness of their async behavior.
		// Let them build first to avoid game-thread stalls as much as possible.
		TFunction<EQueuedWorkPriority(EQueuedWorkPriority)> PriorityMapper = [](EQueuedWorkPriority) { return EQueuedWorkPriority::Highest; };

		// Skinned assets will be scheduled on the asset thread pool, where concurrency limits might by dynamically adjusted depending on memory constraints.
		GSkinnedAssetThreadPool = new FQueuedThreadPoolDynamicWrapper(FAssetCompilingManager::Get().GetThreadPool(), -1, PriorityMapper);

		AsyncCompilationHelpers::BindThreadPoolToCVar(
			GSkinnedAssetThreadPool,
			CVarAsyncSkinnedAssetStandard.AsyncCompilation,
			CVarAsyncSkinnedAssetStandard.AsyncCompilationResume,
			CVarAsyncSkinnedAssetStandard.AsyncCompilationMaxConcurrency
		);
	}

	return GSkinnedAssetThreadPool;
}

void FSkinnedAssetCompilingManager::Shutdown()
{
	bHasShutdown = true;
	if (GetNumRemainingJobs())
	{
		check(IsInGameThread());
		TRACE_CPUPROFILER_EVENT_SCOPE(FSkinnedAssetCompilingManager::Shutdown)

		if (GetNumRemainingJobs())
		{
			TArray<USkinnedAsset*> PendingSkinnedAssets;
			PendingSkinnedAssets.Reserve(GetNumRemainingJobs());

			auto CancelAndCollect = [&PendingSkinnedAssets](TSet<TWeakObjectPtr<USkinnedAsset>>& Set)
			{
				for (TWeakObjectPtr<USkinnedAsset>& WeakSkinnedAsset : Set)
				{
					if (WeakSkinnedAsset.IsValid())
					{
						USkinnedAsset* SkinnedAsset = WeakSkinnedAsset.Get();
						if (!SkinnedAsset->IsAsyncTaskComplete())
						{
							if (SkinnedAsset->AsyncTask->Cancel())
							{
								SkinnedAsset->AsyncTask.Reset();
							}
						}

						if (SkinnedAsset->AsyncTask)
						{
							PendingSkinnedAssets.Add(SkinnedAsset);
						}
					}
				}
			};

			CancelAndCollect(RegisteredSkinnedAsset);
			CancelAndCollect(SkinnedAssetsWithPendingDependencies);

			if (!PendingSkinnedAssets.IsEmpty())
			{
				FinishCompilation(PendingSkinnedAssets);
			}
		}
	}

	FCoreUObjectDelegates::PostReachabilityAnalysis.Remove(PostReachabilityAnalysisHandle);
	FCoreUObjectDelegates::GetPreGarbageCollectDelegate().Remove(PreGarbageCollectHandle);
}

bool FSkinnedAssetCompilingManager::IsAsyncCompilationEnabled() const
{
	if (bHasShutdown)
	{
		return false;
	}

	return CVarAsyncSkinnedAssetStandard.AsyncCompilation.GetValueOnAnyThread() != 0;
}

TRACE_DECLARE_INT_COUNTER(QueuedSkinnedAssetCompilation, TEXT("AsyncCompilation/QueuedSkinnedAsset"));
void FSkinnedAssetCompilingManager::UpdateCompilationNotification()
{
	TRACE_COUNTER_SET(QueuedSkinnedAssetCompilation, GetNumRemainingJobs());
	Notification->Update(GetNumRemainingJobs());
}

void FSkinnedAssetCompilingManager::PostCompilation(TArrayView<USkinnedAsset* const> InSkinnedAssets)
{
	if (InSkinnedAssets.Num())
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(OnAssetPostCompileEvent);

		TArray<FAssetCompileData> AssetsData;
		AssetsData.Reserve(InSkinnedAssets.Num());

		for (USkinnedAsset* SkinnedAsset : InSkinnedAssets)
		{
			AssetsData.Emplace(SkinnedAsset);
		}

		FAssetCompilingManager::Get().OnAssetPostCompileEvent().Broadcast(AssetsData);

		// Schedule compilations that were dependent upon others
		SchedulePendingCompilations();
	}
}

void FSkinnedAssetCompilingManager::PostCompilation(USkinnedAsset* SkinnedAsset)
{
	using namespace SkinnedAssetCompilingManagerImpl;
	
	// If AsyncTask is null here, the task got canceled so we don't need to do anything
	if (SkinnedAsset->AsyncTask)
	{
		check(IsInGameThread());
		TRACE_CPUPROFILER_EVENT_SCOPE(FSkinnedAssetCompilingManager::PostCompilation);

		UE_LOGF(LogSkinnedAsset, Verbose, "Refreshing skinned asset %ls because it is ready", *SkinnedAsset->GetName());

		FObjectCacheContextScope ObjectCacheScope;

		// The scope is important here to destroy the FSkinnedAssetAsyncBuildScope before broadcasting events
		{
			// Acquire the async task locally to protect against re-entrance
			TUniquePtr<FSkinnedAssetAsyncBuildTask> LocalAsyncTask = MoveTemp(SkinnedAsset->AsyncTask);
			LocalAsyncTask->EnsureCompletion();

			// if it has dependencies, remove it from each dependent's reverse lookup
			for (USkinnedAsset* Dependency : SkinnedAsset->GetSkinnedAssetDependencies())
			{
				if (Dependency)
				{
					if (TSet<TWeakObjectPtr<USkinnedAsset>>* ReverseDeps = ReverseDependencyLookup.Find(Dependency))
					{
						ReverseDeps->Remove(SkinnedAsset);
						if (ReverseDeps->IsEmpty())
						{
							ReverseDependencyLookup.Remove(Dependency);
						}
					}
				}
			}

			FSkinnedAssetAsyncBuildScope AsyncBuildScope(SkinnedAsset);

			if (LocalAsyncTask->GetTask().PostLoadContext.IsSet())
			{
				SkinnedAsset->FinishPostLoadInternal(*LocalAsyncTask->GetTask().PostLoadContext);

				LocalAsyncTask->GetTask().PostLoadContext.Reset();
			}

			if (LocalAsyncTask->GetTask().BuildContext.IsSet())
			{
				SkinnedAsset->FinishBuildInternal(*LocalAsyncTask->GetTask().BuildContext);

				LocalAsyncTask->GetTask().BuildContext.Reset();
			}

			if (LocalAsyncTask->GetTask().AsyncTaskContext.IsSet())
			{
				SkinnedAsset->FinishAsyncTaskInternal(*LocalAsyncTask->GetTask().AsyncTaskContext);

				LocalAsyncTask->GetTask().AsyncTaskContext.Reset();
			}
		}

		for (ISkinnedMeshComponent* Component : ObjectCacheScope.GetContext().GetSkinnedMeshComponents(SkinnedAsset))
		{
			Component->PostAssetCompilation();
		}

		// Calling this delegate during app exit might be quite dangerous and lead to crash
		// if the content browser wants to refresh a thumbnail it might try to load a package
		// which will then fail due to various reasons related to the editor shutting down.
		// Triggering this callback while garbage collecting can also result in listeners trying to look up objects
		if (!GExitPurge && !IsGarbageCollecting())
		{
			// Generate an empty property changed event, to force the asset registry tag
			// to be refreshed now that RenderData is available.
			FPropertyChangedEvent EmptyPropertyChangedEvent(nullptr);
			FCoreUObjectDelegates::OnObjectPropertyChanged.Broadcast(SkinnedAsset, EmptyPropertyChangedEvent);
		}
	}
}

void FSkinnedAssetCompilingManager::SchedulePendingCompilations()
{
	TArray<USkinnedAsset*> ReadyToSchedule;
	for (auto It = SkinnedAssetsWithPendingDependencies.CreateIterator(); It; ++It)
	{
		if (USkinnedAsset* SkinnedAsset = It->Get())
		{
			if (SkinnedAsset->HasAnyDependenciesCompiling())
			{
				continue;
			}
			ReadyToSchedule.Emplace(SkinnedAsset);
		}
		It.RemoveCurrent();
	}

	if (ReadyToSchedule.Num() == 0)
	{
		return;
	}

	for (USkinnedAsset* SkinnedAsset : ReadyToSchedule)
	{
		// The mesh's task hasn't been kicked yet, so start it now
		check(SkinnedAsset->AsyncTask != nullptr && SkinnedAsset->AsyncTask->IsIdle());
		SkinnedAsset->AsyncTask->StartBackgroundTask(
			GetThreadPool(),
			GetBasePriority(SkinnedAsset),
			EQueuedWorkFlags::DoNotRunInsideBusyWait,
			SkinnedAsset->GetRequiredMemoryForCompile(),
			TEXT("SkinnedAsset")
		);
	}

	// Add the skinned assets that are now in progress
	AddSkinnedAssets(ReadyToSchedule);
}

bool FSkinnedAssetCompilingManager::IsAsyncCompilationAllowed(USkinnedAsset* SkinnedAsset) const
{
	return IsAsyncCompilationEnabled();
}

FSkinnedAssetCompilingManager& FSkinnedAssetCompilingManager::Get()
{
	static FSkinnedAssetCompilingManager Singleton;
	return Singleton;
}

int32 FSkinnedAssetCompilingManager::GetNumRemainingJobs() const
{
	return RegisteredSkinnedAsset.Num() + SkinnedAssetsWithPendingDependencies.Num();
}

void FSkinnedAssetCompilingManager::AddSkinnedAssets(TArrayView<USkinnedAsset* const> InSkinnedAssets)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FSkinnedAssetCompilingManager::AddSkinnedAssets)
	check(IsInGameThread());

	// Wait until we gather enough mesh to process
	// to amortize the cost of scanning components
	//ProcessSkinnedAssets(32 /* MinBatchSize */);

	for (USkinnedAsset* SkinnedAsset : InSkinnedAssets)
	{
		check(SkinnedAsset->AsyncTask != nullptr);
		RegisteredSkinnedAsset.Emplace(SkinnedAsset);
	}

	UpdateCompilationNotification();
}

void FSkinnedAssetCompilingManager::AddSkinnedAssetsWithDependencies(TArrayView<USkinnedAsset* const> InSkinnedAssets)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FSkinnedAssetCompilingManager::AddSkinnedAssets)
	check(IsInGameThread());

	for (USkinnedAsset* SkinnedAsset : InSkinnedAssets)
	{
		for (USkinnedAsset* Dependency : SkinnedAsset->GetSkinnedAssetDependencies())
		{
			if (Dependency)
			{
				TSet<TWeakObjectPtr<USkinnedAsset>>& ReverseLookup = ReverseDependencyLookup.FindOrAdd(Dependency);
				ReverseLookup.Add(SkinnedAsset);
			}
		}

		check(SkinnedAsset->AsyncTask != nullptr);
		if (SkinnedAsset->AsyncTask->IsIdle())
		{
			// The task couldn't be started yet due to compiling dependencies, so add it to the pending list
			SkinnedAssetsWithPendingDependencies.Add(SkinnedAsset);
		}
		else
		{
			RegisteredSkinnedAsset.Emplace(SkinnedAsset);
		}
	}

	UpdateCompilationNotification();
}



void FSkinnedAssetCompilingManager::FinishCompilation(TArrayView<USkinnedAsset* const> InSkinnedAssets, const FFinishCompilationOptions& Options)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FSkinnedAssetCompilingManager::FinishCompilation);

	// Allow calls from any thread if the assets are already finished compiling.
	if (!Options.bIncludeDependentAssets && Algo::NoneOf(InSkinnedAssets, &USkinnedAsset::IsCompiling))
	{
		return;
	}

	check(IsInGameThread());

	TArray<USkinnedAsset*> PendingSkinnedAssets, NextPendingSkinnedAssets;
	PendingSkinnedAssets.Reserve(InSkinnedAssets.Num());

	auto FilterIntoPendingLists =
		[this, &PendingSkinnedAssets, &NextPendingSkinnedAssets, &Options] (TArrayView<USkinnedAsset* const> Assets)
	{
		PendingSkinnedAssets.SetNum(0, EAllowShrinking::No);
		NextPendingSkinnedAssets.SetNum(0, EAllowShrinking::No);

		for (USkinnedAsset* SkinnedAsset : Assets)
		{
			if (RegisteredSkinnedAsset.Contains(SkinnedAsset))
			{
				PendingSkinnedAssets.AddUnique(SkinnedAsset);
			}
			else if (SkinnedAssetsWithPendingDependencies.Contains(SkinnedAsset))
			{
				// Add it to the next wave of meshes to finish
				NextPendingSkinnedAssets.AddUnique(SkinnedAsset);
			}

			if (Options.bIncludeDependentAssets)
			{
				// If we're stalling on the mesh compilation because we're about to edit the mesh, we have to stall on any
				// compiling mesh that depends on us as well, to make sure we don't write to the mesh while it's being read
				if (TSet<TWeakObjectPtr<USkinnedAsset>>* ReverseLookup = ReverseDependencyLookup.Find(SkinnedAsset))
				{
					for (TWeakObjectPtr<USkinnedAsset>& ReverseDependency : *ReverseLookup)
					{
						if (USkinnedAsset* Dependent = ReverseDependency.Get())
						{
							NextPendingSkinnedAssets.AddUnique(Dependent);
						}
					}
				}
			}
		}

		// Recursively filter in all dependencies
		for (int32 NextPendingIndex = 0; NextPendingIndex < NextPendingSkinnedAssets.Num(); ++NextPendingIndex)
		{
			USkinnedAsset* SkinnedAsset = NextPendingSkinnedAssets[NextPendingIndex];
			for (USkinnedAsset* Dependency : SkinnedAsset->GetSkinnedAssetDependencies())
			{
				if (RegisteredSkinnedAsset.Contains(Dependency))
				{
					PendingSkinnedAssets.AddUnique(Dependency);
				}
				else if (SkinnedAssetsWithPendingDependencies.Contains(Dependency))
				{
					NextPendingSkinnedAssets.AddUnique(Dependency);
				}
			}
		}

		if (PendingSkinnedAssets.Num() == 0 && NextPendingSkinnedAssets.Num() > 0)
		{
			PendingSkinnedAssets = MoveTemp(NextPendingSkinnedAssets);
		}
	};

	bool bFinishedAny = false;
	FilterIntoPendingLists(InSkinnedAssets);
	while (PendingSkinnedAssets.Num())
	{
		class FCompilableSkinnedAsset : public AsyncCompilationHelpers::TCompilableAsyncTask<FSkinnedAssetAsyncBuildTask>
		{
		public:
			FCompilableSkinnedAsset(USkinnedAsset* InSkinnedAsset)
				: SkinnedAsset(InSkinnedAsset)
			{
			}

			FSkinnedAssetAsyncBuildTask* GetAsyncTask() override
			{
				return SkinnedAsset->AsyncTask.Get();
			}

			TStrongObjectPtr<USkinnedAsset> SkinnedAsset;
			FName GetName() override { return SkinnedAsset->GetFName(); }
		};

		TArray<FCompilableSkinnedAsset> CompilableSkinnedAsset(PendingSkinnedAssets);

		FObjectCacheContextScope ObjectCacheScope;
		AsyncCompilationHelpers::FinishCompilation(
			[&CompilableSkinnedAsset](int32 Index) -> AsyncCompilationHelpers::ICompilable& { return CompilableSkinnedAsset[Index]; },
			CompilableSkinnedAsset.Num(),
			LOCTEXT("SkinnedAssets", "Skinned Assets"),
			LogSkinnedAsset,
			[this](AsyncCompilationHelpers::ICompilable* Object)
			{
				USkinnedAsset* SkinnedAsset = static_cast<FCompilableSkinnedAsset*>(Object)->SkinnedAsset.Get();
				PostCompilation(SkinnedAsset);
				RegisteredSkinnedAsset.Remove(SkinnedAsset);
			}
		);

		PostCompilation(PendingSkinnedAssets);
		
		TArray<USkinnedAsset*> Temp = MoveTemp(NextPendingSkinnedAssets);
		FilterIntoPendingLists(Temp);
		bFinishedAny = true;
	}
	
	// Sanity check - if no dependencies are pending, it should have already been put in the active list
	check(NextPendingSkinnedAssets.Num() == 0);

	if (bFinishedAny)
	{
		UpdateCompilationNotification();
	}
}

void FSkinnedAssetCompilingManager::FinishCompilationsForGame()
{
	
}

void FSkinnedAssetCompilingManager::FinishAllCompilation()
{
	check(IsInGameThread());
	TRACE_CPUPROFILER_EVENT_SCOPE(FSkinnedAssetCompilingManager::FinishAllCompilation)

	if (GetNumRemainingJobs())
	{
		TArray<USkinnedAsset*> PendingSkinnedAssets;
		PendingSkinnedAssets.Reserve(GetNumRemainingJobs());

		auto CollectAllValid = [&PendingSkinnedAssets](TSet<TWeakObjectPtr<USkinnedAsset>>& Set)
		{
			for (TWeakObjectPtr<USkinnedAsset>& SkinnedAsset : Set)
			{
				if (SkinnedAsset.IsValid())
				{
					PendingSkinnedAssets.Add(SkinnedAsset.Get());
				}
			}
		};

		CollectAllValid(RegisteredSkinnedAsset);
		CollectAllValid(SkinnedAssetsWithPendingDependencies);

		if (!PendingSkinnedAssets.IsEmpty())
		{
			FinishCompilation(PendingSkinnedAssets);
		}
	}
}

void FSkinnedAssetCompilingManager::FinishCompilationForObjects(TArrayView<UObject* const> InObjects)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FSkinnedAssetCompilingManager::FinishCompilationForObjects);

	TSet<USkinnedAsset*> SkinnedAssets;
	for (UObject* Object : InObjects)
	{
		if (USkinnedAsset* SkinnedAsset = Cast<USkinnedAsset>(Object))
		{
			SkinnedAssets.Add(SkinnedAsset);
		}
		else if (USkinnedMeshComponent* SkinnedMeshComponent = Cast<USkinnedMeshComponent>(Object))
		{
			if (SkinnedMeshComponent->GetSkinnedAsset())
			{
				SkinnedAssets.Add(SkinnedMeshComponent->GetSkinnedAsset());
			}
		}
	}

	if (SkinnedAssets.Num())
	{
		FFinishCompilationOptions Options;
		Options.bIncludeDependentAssets = true;
		FinishCompilation(SkinnedAssets.Array(), Options);
	}
}

void FSkinnedAssetCompilingManager::Reschedule()
{

}

void FSkinnedAssetCompilingManager::ProcessSkinnedAssets(bool bLimitExecutionTime, int32 MinBatchSize)
{
	using namespace SkinnedAssetCompilingManagerImpl;
	TRACE_CPUPROFILER_EVENT_SCOPE(FSkinnedAssetCompilingManager::ProcessSkinnedAssets);
	const int32 NumRemainingMeshes = GetNumRemainingJobs();
	// Spread out the load over multiple frames but if too many meshes, convergence is more important than frame time
	const int32 MaxMeshUpdatesPerFrame = bLimitExecutionTime ? FMath::Max(64, NumRemainingMeshes / 10) : INT32_MAX;

	FObjectCacheContextScope ObjectCacheScope;
	if (NumRemainingMeshes && NumRemainingMeshes >= MinBatchSize)
	{
		TSet<USkinnedAsset*> SkinnedAssetsToProcess;
		for (TWeakObjectPtr<USkinnedAsset>& SkinnedAsset : RegisteredSkinnedAsset)
		{
			if (SkinnedAsset.IsValid())
			{
				SkinnedAssetsToProcess.Add(SkinnedAsset.Get());
			}
		}

		{
			TRACE_CPUPROFILER_EVENT_SCOPE(ProcessFinishedSkinnedAssets);

			const double TickStartTime = FPlatformTime::Seconds();

			TSet<TWeakObjectPtr<USkinnedAsset>> SkinnedAssetsToPostpone;
			TArray<USkinnedAsset*> ProcessedSkinnedAssets;
			if (SkinnedAssetsToProcess.Num())
			{
				for (USkinnedAsset* SkinnedAsset : SkinnedAssetsToProcess)
				{
					const bool bHasMeshUpdateLeft = ProcessedSkinnedAssets.Num() <= MaxMeshUpdatesPerFrame;
					if (bHasMeshUpdateLeft && SkinnedAsset->IsAsyncTaskComplete())
					{
						PostCompilation(SkinnedAsset);
						ProcessedSkinnedAssets.Add(SkinnedAsset);
					}
					else
					{
						SkinnedAssetsToPostpone.Emplace(SkinnedAsset);
					}
				}
			}

			RegisteredSkinnedAsset = MoveTemp(SkinnedAssetsToPostpone);
			
			if (ProcessedSkinnedAssets.Num())
			{
				PostCompilation(ProcessedSkinnedAssets);
			}
		}
	}
}

void FSkinnedAssetCompilingManager::ProcessAsyncTasks(bool bLimitExecutionTime)
{
	FObjectCacheContextScope ObjectCacheScope;
	FinishCompilationsForGame();

	Reschedule();

	ProcessSkinnedAssets(bLimitExecutionTime);

	UpdateCompilationNotification();
}

void FSkinnedAssetCompilingManager::OnPostReachabilityAnalysis()
{
	if (GetNumRemainingJobs())
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FSkinnedAssetCompilingManager::CancelUnreachableMeshes);

		TArray<USkinnedAsset*> PendingSkinnedMeshes;
		PendingSkinnedMeshes.Reserve(GetNumRemainingJobs());

		auto CancelOrCollectUnreachable = [&PendingSkinnedMeshes](TSet<TWeakObjectPtr<USkinnedAsset>>& Set)
		{
			for (auto Iterator = Set.CreateIterator(); Iterator; ++Iterator)
			{
				USkinnedAsset* SkinnedMesh = Iterator->GetEvenIfUnreachable();
				if (SkinnedMesh && SkinnedMesh->IsUnreachable())
				{
					UE_LOGF(LogSkinnedAsset, Verbose, "Cancelling skinned mesh %ls async compilation because it's being garbage collected", *SkinnedMesh->GetName());

					if (SkinnedMesh->TryCancelAsyncTasks())
					{
						Iterator.RemoveCurrent();
					}
					else
					{
						PendingSkinnedMeshes.Add(SkinnedMesh);
					}
				}
			}
		};

		CancelOrCollectUnreachable(RegisteredSkinnedAsset);
		CancelOrCollectUnreachable(SkinnedAssetsWithPendingDependencies);

		if (!PendingSkinnedMeshes.IsEmpty())
		{
			FinishCompilation(PendingSkinnedMeshes);
		}
	}
}

void FSkinnedAssetCompilingManager::OnPreGarbageCollect()
{
	FinishAllCompilation();
}

#endif // #if WITH_EDITOR

#undef LOCTEXT_NAMESPACE
