// Copyright Epic Games, Inc. All Rights Reserved.

#include "SkinnedMeshSceneProxyDesc.h"

#include "SkeletalMeshSceneProxy.h"
#include "SkeletalRenderCPUSkin.h"
#include "SkeletalRenderGPUSkin.h"
#include "SkeletalRenderNanite.h"
#include "SkeletalRenderStatic.h"
#include "Materials/MaterialInstanceDynamic.h"
#include "Engine/Engine.h"
#include "Engine/MaterialOverlayHelper.h"
#include "Engine/SkinnedAssetCommon.h"
#include "Engine/SkinnedAsset.h"
#include "PhysicsEngine/PhysicsAsset.h"
#include "Components/SkinnedMeshComponent.h"
#include "SceneInterface.h"
#include "GPUSkinCache.h"
#include "ContentStreaming.h"
#include "SkinnedMeshComponentHelper.h"

#if WITH_EDITORONLY_DATA
#include "Animation/DebugSkelMeshComponent.h"
#endif

FSkinnedMeshSceneProxyDesc::FSkinnedMeshSceneProxyDesc(const USkinnedMeshComponent* Component)
{ 
	InitializeFromSkinnedMeshComponent(Component);
}

void FSkinnedMeshSceneProxyDesc::InitializeFromSkinnedMeshComponent(const USkinnedMeshComponent* InComponent)
{
	InitializeFromPrimitiveComponent(InComponent);
	bForceWireframe = InComponent->bForceWireframe;
	bCanHighlightSelectedSections = InComponent->bCanHighlightSelectedSections;
	bRenderStatic = InComponent->bRenderStatic;
	bPerBoneMotionBlur = InComponent->bPerBoneMotionBlur;
	bCastCapsuleDirectShadow = InComponent->bCastCapsuleDirectShadow;
	bCastCapsuleIndirectShadow = InComponent->bCastCapsuleIndirectShadow;
	bAllowAlwaysVisible = InComponent->VisibilityBasedAnimTickOption != EVisibilityBasedAnimTickOption::OnlyTickPoseWhenRendered;
#if UE_ENABLE_DEBUG_DRAWING
	bDrawDebugSkeleton = InComponent->ShouldDrawDebugSkeleton();
#endif
	bCPUSkinning = InComponent->ShouldCPUSkin();
#if UE_ENABLE_DEBUG_DRAWING
	DebugDrawColor = InComponent->GetDebugDrawColor();
#endif
#if WITH_EDITORONLY_DATA
	bClothPainting = InComponent->IsA<UDebugSkelMeshComponent>();
#endif
	if (GetScene())
	{
		MaterialRelevance = InComponent->GetMaterialRelevance(GetScene()->GetShaderPlatform());
	}
	StreamingDistanceMultiplier = InComponent->StreamingDistanceMultiplier;
	NanitePixelProgrammableDistance = InComponent->NanitePixelProgrammableDistance;
	WorldPositionOffsetDisableDistance = InComponent->WorldPositionOffsetDisableDistance;
	bEvaluateWorldPositionOffset = InComponent->EvaluateWorldPositionOffset();
	bDisallowNanite = InComponent->IsDisallowNanite();
	bForceDisableNanite = InComponent->IsForceDisableNanite();
	CapsuleIndirectShadowMinVisibility = InComponent->CapsuleIndirectShadowMinVisibility;
	OverlayMaterialMaxDrawDistance = InComponent->GetOverlayMaterialMaxDrawDistance();
	PredictedLODLevel = InComponent->GetPredictedLODLevel();
	MinLODLevel = InComponent->ComputeMinLOD();
	const ILODSyncInterface* LODInterface = InComponent;
	ForceStreamedLOD = LODInterface->GetForceStreamedLOD();
	MaxDistanceFactor = InComponent->MaxDistanceFactor;
	ComponentScale = InComponent->GetComponentScale();
	LODInfo = InComponent->LODInfo;
	MeshObject = InComponent->MeshObject;
	PreviousMeshObject = InComponent->PreviousMeshObject;
	SkinnedAsset = InComponent->GetSkinnedAsset();
	PhysicsAsset = InComponent->GetPhysicsAsset();
	OverlayMaterial = InComponent->GetOverlayMaterial();
	InComponent->GetMaterialSlotsOverlayMaterial(MaterialSlotsOverlayMaterial);
	MeshDeformerInstances = &InComponent->GetMeshDeformerInstances();
	OverrideMaterials = InComponent->OverrideMaterials;
	SkinCacheUsage = InComponent->SkinCacheUsage;
#if WITH_EDITOR
	SectionIndexPreview = InComponent->GetSectionPreview();
	MaterialIndexPreview = InComponent->GetMaterialPreview();
	SelectedEditorSection = InComponent->GetSelectedEditorSection();
	SelectedEditorMaterial = InComponent->GetSelectedEditorMaterial();
#endif
	bSortTriangles = InComponent->bSortTriangles;
}

USkinnedAsset* FSkinnedMeshSceneProxyDesc::GetSkinnedAsset() const
{
	return SkinnedAsset;
}

UPhysicsAsset* FSkinnedMeshSceneProxyDesc::GetPhysicsAsset() const
{
	return PhysicsAsset;
}

bool FSkinnedMeshSceneProxyDesc::ShouldDrawDebugSkeleton() const
{
	return bDrawDebugSkeleton;
}

const TOptional<FLinearColor>& FSkinnedMeshSceneProxyDesc::GetDebugDrawColor() const
{
	return DebugDrawColor;
}


UMeshDeformerInstance* FSkinnedMeshSceneProxyDesc::GetMeshDeformerInstanceForLOD(int32 LODIndex) const
{
	if ((!MeshDeformerInstances) || !MeshDeformerInstances->InstanceIndexForLOD.IsValidIndex(LODIndex))
	{
		return nullptr;
	}
	
	const int8 InstanceIndex = MeshDeformerInstances->InstanceIndexForLOD[LODIndex];
	if (InstanceIndex == INDEX_NONE)
	{
		// Don't use a deformer for this LOD
		return nullptr;
	}

	check(MeshDeformerInstances->DeformerInstances.IsValidIndex(InstanceIndex));
	return MeshDeformerInstances->DeformerInstances[InstanceIndex];
}

void FSkinnedMeshSceneProxyDesc::GetPreSkinnedLocalBounds(FBoxSphereBounds& OutBounds) const
{
	if (GetSkinnedAsset())
	{
		// Get the Pre-skinned bounds from the skeletal mesh. Note that these bounds are the "ExtendedBounds", so they can be tweaked on the SkeletalMesh   
		OutBounds = GetSkinnedAsset()->GetBounds();
	}
	else
	{
		// Fall back
		OutBounds = FBoxSphereBounds(ForceInitToZero);
	}
}

int32 FSkinnedMeshSceneProxyDesc::GetBoneIndex(FName BoneName) const
{
	return (BoneName != NAME_None && GetSkinnedAsset()) ?  GetSkinnedAsset()->GetRefSkeleton().FindBoneIndex(BoneName) : INDEX_NONE;
}

// Deprecated in 5.7
FMaterialRelevance FSkinnedMeshSceneProxyDesc::GetMaterialRelevance(ERHIFeatureLevel::Type InFeatureLevel) const
{
	return GetMaterialRelevance(GetFeatureLevelShaderPlatform_Checked(InFeatureLevel));
}

FMaterialRelevance FSkinnedMeshSceneProxyDesc::GetMaterialRelevance(EShaderPlatform InShaderPlatform) const
{
	return MaterialRelevance;
}

float FSkinnedMeshSceneProxyDesc::GetOverlayMaterialMaxDrawDistance() const
{
	return OverlayMaterialMaxDrawDistance;
}
UMaterialInterface* FSkinnedMeshSceneProxyDesc::GetOverlayMaterial() const
{
	return OverlayMaterial;
}

UMaterialInterface* FSkinnedMeshSceneProxyDesc::GetMaterial(int32 MaterialIndex) const
{
	return FSkinnedMeshComponentHelper::GetMaterial(*this, MaterialIndex);
}

void FSkinnedMeshSceneProxyDesc::GetUsedMaterials(TArray<UMaterialInterface*>& OutMaterials, bool bGetDebugMaterials /* = false */) const
{
	if (GetSkinnedAsset())
	{
		// The max number of materials used is the max of the materials on the skeletal mesh and the materials on the mesh component
		const int32 NumMaterials = FMath::Max(GetSkinnedAsset()->GetMaterials().Num(), OverrideMaterials.Num() );
		for( int32 MatIdx = 0; MatIdx < NumMaterials; ++MatIdx )
		{
			// GetMaterial will determine the correct material to use for this index.
			UMaterialInterface* MaterialInterface = GetMaterial( MatIdx );
			OutMaterials.Add( MaterialInterface );
		}

		bool bUseGlobalMeshOverlayMaterial = false;
		FMaterialOverlayHelper::AppendAllOverlayMaterial(MaterialSlotsOverlayMaterial, OutMaterials, bUseGlobalMeshOverlayMaterial);
		if(bUseGlobalMeshOverlayMaterial)
		{
			UMaterialInterface* OverlayMaterialInterface = GetOverlayMaterial();
			if (OverlayMaterialInterface != nullptr)
			{
				OutMaterials.Add(OverlayMaterialInterface);
			}
		}
	}

	if (bGetDebugMaterials)
	{
#if WITH_EDITOR
		if (UPhysicsAsset* PhysicsAssetForDebug = GetPhysicsAsset())
		{
			PhysicsAssetForDebug->GetUsedMaterials(OutMaterials);
		}
#endif

#if WITH_EDITORONLY_DATA
		if (bClothPainting)
		{
			OutMaterials.Add(GEngine->ClothPaintMaterialInstance);
			OutMaterials.Add(GEngine->ClothPaintMaterialWireframeInstance);
			OutMaterials.Add(GEngine->ClothPaintOpaqueMaterialInstance);
			OutMaterials.Add(GEngine->ClothPaintOpaqueMaterialWireframeInstance);
		}
#endif
	}
}

extern bool ShouldRenderNaniteSkinnedMeshes();

bool FSkinnedMeshSceneProxyDesc::HasValidNaniteData() const
{
	return FSkinnedMeshComponentHelper::HasValidNaniteData(*this);
}

bool FSkinnedMeshSceneProxyDesc::ShouldNaniteSkin() const
{
	return FSkinnedMeshComponentHelper::ShouldNaniteSkin(*this);
}

FSkeletalMeshObject* FSkinnedMeshSceneProxyDesc::CreateMeshObject(const FSkinnedMeshSceneProxyDesc& Desc)
{
	FSkeletalMeshRenderData* SkelMeshRenderData = Desc.GetSkinnedAsset()->GetResourceForRendering();
	const int32 MinLODIndex = SkelMeshRenderData->LODRenderData.Num()-1;
	if (Desc.ShouldNaniteSkin() && !Desc.ShouldCPUSkin() /* Needed for calls to GetCPUSkinnedVertices() */)
	{
		FSkeletalMeshObjectNanite* NaniteMeshObject = ::new FSkeletalMeshObjectNanite(Desc, SkelMeshRenderData, Desc.FeatureLevel);
		if (NaniteMeshObject->HasValidMaterials())
		{
			return NaniteMeshObject;
		}
		else
		{
			// TODO: This should run outside of statestream as well because otherwise resources are leaked and an assert happens inside
			// SkeletalMeshUpdater. But since there are enqueued commands accessing the NaniteMeshObject, then I don't know if this will work on game thread
			#if WITH_STATE_STREAM
			NaniteMeshObject->ReleaseResources();
			#endif

			delete NaniteMeshObject;
		}
	}

	if (Desc.bRenderStatic)
	{
		// GPU skin vertex buffer + LocalVertexFactory
		return ::new FSkeletalMeshObjectStatic(Desc, SkelMeshRenderData, Desc.FeatureLevel);
	}
	else if (Desc.ShouldCPUSkin())
	{
		return ::new FSkeletalMeshObjectCPUSkin(Desc, SkelMeshRenderData, Desc.FeatureLevel);
	}

	return ::new FSkeletalMeshObjectGPUSkin(Desc, SkelMeshRenderData,  Desc.FeatureLevel);
}

FPrimitiveSceneProxy* FSkinnedMeshSceneProxyDesc::CreateSceneProxy(const FSkinnedMeshSceneProxyDesc& Desc, bool bHideSkin)
{
	LLM_SCOPE(ELLMTag::SkeletalMesh);
	ERHIFeatureLevel::Type SceneFeatureLevel = Desc.FeatureLevel;
	FPrimitiveSceneProxy* Result = nullptr;
	FSkeletalMeshRenderData* SkelMeshRenderData = Desc.GetSkinnedAsset()->GetResourceForRendering();

	FSkeletalMeshObject* MeshObject = Desc.MeshObject;

	// Only create a scene proxy for rendering if properly initialized
	if (SkelMeshRenderData &&
		SkelMeshRenderData->LODRenderData.IsValidIndex(Desc.PredictedLODLevel) &&
		!bHideSkin &&
		MeshObject)
	{
		if (MeshObject->IsNaniteMesh())
		{
			check(MeshObject->GetNaniteMaterials());
			Result = ::new Nanite::FSkinnedSceneProxy(*MeshObject->GetNaniteMaterials(), Desc, SkelMeshRenderData);
		}
		else
		{
			Result = ::new FSkeletalMeshSceneProxy(Desc, SkelMeshRenderData);
		}
	}

	return Result;
}

bool FSkinnedMeshSceneProxyDesc::IsSkinCacheAllowed(int32 LodIdx) const
{
	const bool bGlobalDefault = GetSkinCacheDefaultBehavior() == ESkinCacheDefaultBehavior::Inclusive;

	if (!Scene || !Scene->GetGPUSkinCache() || !FGPUSkinCache::IsGPUSkinCacheEnabled(Scene->GetFeatureLevel()))
	{
		return false;
	}

	if (GetMeshDeformerInstanceForLOD(LodIdx) != nullptr)
	{
		// Disable skin cache if a mesh deformer is in use.
		// Any animation buffers are expected to be owned by the MeshDeformer.
		return false;
	}

	if (!GetSkinnedAsset())
	{
		return bGlobalDefault;
	}

	if (FSkeletalMeshRenderData* SkelMeshRenderData = GetSkinnedAsset()->GetResourceForRendering())
	{
		if (SkelMeshRenderData->bSupportRayTracing && FGPUSkinCache::IsGPUSkinCacheRayTracingSupported(Scene->GetFeatureLevel()) && IsRayTracingEnabled())
		{
			return true;
		}
	}

	FSkeletalMeshLODInfo* LodInfo = GetSkinnedAsset()->GetLODInfo(LodIdx);
	if (!LodInfo)
	{
		return bGlobalDefault;
	}

	bool bLodEnabled = LodInfo->SkinCacheUsage == ESkinCacheUsage::Auto ?
		bGlobalDefault :
		LodInfo->SkinCacheUsage == ESkinCacheUsage::Enabled;

	if (!SkinCacheUsage.IsValidIndex(LodIdx))
	{
		return bLodEnabled;
	}

	bool bComponentEnabled = SkinCacheUsage[LodIdx] == ESkinCacheUsage::Auto ? 
		bLodEnabled :
		SkinCacheUsage[LodIdx] == ESkinCacheUsage::Enabled;

	return bComponentEnabled;
}

UMeshDeformerInstance* FSkinnedMeshSceneProxyDesc::GetMeshDeformerInstance() const
{
	return (MeshDeformerInstances && MeshDeformerInstances->DeformerInstances.Num() > 0)? MeshDeformerInstances->DeformerInstances[0] : nullptr;
}

bool FSkinnedMeshSceneProxyDynamicData::IsSkinCacheAllowed(int32 LodIdx, const USkinnedAsset* InSkinnedAsset) const
{
	const bool bGlobalDefault = GetSkinCacheDefaultBehavior() == ESkinCacheDefaultBehavior::Inclusive;

	if (!FGPUSkinCache::IsGPUSkinCacheEnabled(GMaxRHIFeatureLevel))
	{
		return false;
	}

	if (bHasMeshDeformerInstance)
	{
		// Disable skin cache if a mesh deformer is in use.
		// Any animation buffers are expected to be owned by the MeshDeformer.
		return false;
	}

	if (!InSkinnedAsset)
	{
		return bGlobalDefault;
	}

	if (FSkeletalMeshRenderData* SkelMeshRenderData = InSkinnedAsset->GetResourceForRendering())
	{
		if (SkelMeshRenderData->bSupportRayTracing && FGPUSkinCache::IsGPUSkinCacheRayTracingSupported(GMaxRHIFeatureLevel) && IsRayTracingEnabled())
		{
			return true;
		}
	}

	const FSkeletalMeshLODInfo* LodInfo = InSkinnedAsset->GetLODInfo(LodIdx);
	if (!LodInfo)
	{
		return bGlobalDefault;
	}

	bool bLodEnabled = LodInfo->SkinCacheUsage == ESkinCacheUsage::Auto ?
		bGlobalDefault :
		LodInfo->SkinCacheUsage == ESkinCacheUsage::Enabled;

	if (!SkinCacheUsage.IsValidIndex(LodIdx))
	{
		return bLodEnabled;
	}

	bool bComponentEnabled = SkinCacheUsage[LodIdx] == ESkinCacheUsage::Auto ? 
		bLodEnabled :
		SkinCacheUsage[LodIdx] == ESkinCacheUsage::Enabled;

	return bComponentEnabled;
}

extern TAutoConsoleVariable<int32> CVarMeshDeformerMaxLod;

int32 FSkinnedMeshSceneProxyDynamicData::GetMeshDeformerMaxLOD() const
{
	const int32 MaxLod = CVarMeshDeformerMaxLod.GetValueOnGameThread();
	return MaxLod >= 0 ? MaxLod : GetNumLODs() - 1;
}

UMeshDeformerInstance* FSkinnedMeshSceneProxyDynamicData::GetMeshDeformerInstanceForLOD(int32 LODIndex) const
{
	if (MeshDeformerInstances == nullptr || !MeshDeformerInstances->InstanceIndexForLOD.IsValidIndex(LODIndex))
	{
		return nullptr;
	}
	
	const int8 InstanceIndex = MeshDeformerInstances->InstanceIndexForLOD[LODIndex];
	if (InstanceIndex == INDEX_NONE)
	{
		// Don't use a deformer for this LOD
		return nullptr;
	}

	check(MeshDeformerInstances->DeformerInstances.IsValidIndex(InstanceIndex));
	return MeshDeformerInstances->DeformerInstances[InstanceIndex];
}

FSkinnedMeshSceneProxyDynamicData::FSkinnedMeshSceneProxyDynamicData(const USkinnedMeshComponent* InSkinnedMeshComponent)
	: FSkinnedMeshSceneProxyDynamicData(InSkinnedMeshComponent, InSkinnedMeshComponent->LeaderPoseComponent.Get())
{}
FSkinnedMeshSceneProxyDynamicData::FSkinnedMeshSceneProxyDynamicData(const USkinnedMeshComponent* InSkinnedMeshComponent, const USkinnedMeshComponent* InLeaderPoseComponent)
	: Name(InSkinnedMeshComponent->GetFName())
	, ClothSimulDataProvider(InSkinnedMeshComponent)
	, MeshDeformerInstances(&InSkinnedMeshComponent->GetMeshDeformerInstances())
	, RefPoseOverride(InSkinnedMeshComponent->GetRefPoseOverride())
	, ExternalMorphSets(InSkinnedMeshComponent->ExternalMorphSets)
	, ComponentSpaceTransforms(InLeaderPoseComponent != nullptr ? InLeaderPoseComponent->GetComponentSpaceTransforms() : InSkinnedMeshComponent->GetComponentSpaceTransforms())
	, PreviousComponentSpaceTransforms(InLeaderPoseComponent != nullptr ? InLeaderPoseComponent->GetPreviousComponentTransformsArray() : InSkinnedMeshComponent->GetPreviousComponentTransformsArray() )
	, BoneVisibilityStates(InLeaderPoseComponent != nullptr ? InLeaderPoseComponent->GetBoneVisibilityStates() : InSkinnedMeshComponent->GetBoneVisibilityStates())
	, PreviousBoneVisibilityStates(InLeaderPoseComponent != nullptr ? InLeaderPoseComponent->GetPreviousBoneVisibilityStates() : InSkinnedMeshComponent->GetPreviousBoneVisibilityStates()) 
	, LeaderBoneMap(InSkinnedMeshComponent->GetLeaderBoneMap())
	, LeaderValidMeshPoseTransforms(InLeaderPoseComponent != nullptr ? &InLeaderPoseComponent->GetValidMeshPoseTransforms() : nullptr)
	, SkinCacheUsage(InSkinnedMeshComponent->SkinCacheUsage)
	, ComponentWorldTransform(InSkinnedMeshComponent->GetComponentTransform())
	, CurrentBoneTransformRevisionNumber(InSkinnedMeshComponent->GetBoneTransformRevisionNumber())
	, PreviousBoneTransformRevisionNumber(InSkinnedMeshComponent->GetPreviousBoneTransformRevisionNumber())
	, CurrentBoneTransformFrame(InSkinnedMeshComponent->GetCurrentBoneTransformFrame())
	, NumLODs(static_cast<uint16>(InSkinnedMeshComponent->GetNumLODs()))
	, bHasLeaderPoseComponent(InLeaderPoseComponent != nullptr)
	, bHasMeshDeformerInstance(InSkinnedMeshComponent->GetMeshDeformerInstance() != nullptr)
	, bRenderStateRecreating(InSkinnedMeshComponent->IsRenderStateRecreating())
	, bDrawInGame(InSkinnedMeshComponent->IsVisible())
	, bCastsHiddenShadow(InSkinnedMeshComponent->CastShadow && InSkinnedMeshComponent->bCastHiddenShadow)
	, bAffectIndirectLightingWhileHidden(InSkinnedMeshComponent->bAffectIndirectLightingWhileHidden)
{
	ensureMsgf(InLeaderPoseComponent == nullptr || InSkinnedMeshComponent->GetLeaderBoneMap().Num() == InSkinnedMeshComponent->GetSkinnedAsset()->GetRefSkeleton().GetNum(),
		TEXT("Leader pose component skeleton doesn't match follower. LeaderPoseComponent: %s SkinnedMeshComponent: %s"),
		InLeaderPoseComponent ? *InLeaderPoseComponent->GetFullName() : *FString("NULL"),
		InSkinnedMeshComponent ? *InSkinnedMeshComponent->GetFullName() : *FString("NULL"));
}

FSkinnedMeshSceneProxyDynamicData::FSkinnedMeshSceneProxyDynamicData() { }

bool FSkinnedMeshSceneProxyDynamicData::IsValidExternalMorphSetLODIndex(uint32 InLODIndex) const
{
	return ExternalMorphSets.IsValidIndex(InLODIndex);
}

const FExternalMorphSets& FSkinnedMeshSceneProxyDynamicData::GetExternalMorphSets(uint32 InLODIndex) const
{
	return ExternalMorphSets[InLODIndex];
}