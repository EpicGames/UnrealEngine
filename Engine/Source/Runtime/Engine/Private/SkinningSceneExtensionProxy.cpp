// Copyright Epic Games, Inc. All Rights Reserved.

#include "SkinningSceneExtensionProxy.h"
#include "SkeletalRenderPublic.h"
#include "SkinningDefinitions.h"
#include "AnimationRuntime.h"
#include "Engine/SkeletalMeshSocket.h"
#include "Rendering/SkeletalMeshRenderData.h"
#include "SkinnedMeshSceneProxyDesc.h"
#include "InstancedSkinnedMeshSceneProxyDesc.h"

extern TAutoConsoleVariable<int32> CVarInstancedSkinnedMeshesForceRefPose;

FSkinningSceneExtensionProxy::FSkinningSceneExtensionProxy(
	const FSkeletalMeshObject* InMeshObject,
	const USkinnedAsset* InSkinnedAsset,
	bool bAllowScaling,
	bool bInIncludeSocketsInBoneMap)
	: SkinnedAsset(InSkinnedAsset)
	, MeshObject(InMeshObject)
{
	FSkeletalMeshRenderData& RenderData = MeshObject->GetSkeletalMeshRenderData();
	MaxBoneInfluenceCount = RenderData.GetNumBoneInfluences();
	BoneTransformStorageMode = MeshObject->GetBoneTransformStorageMode();
	bHasUnifiedBoneMap = RenderData.HasUnifiedBoneMap();
	bHasSocketsInBoneMap = bInIncludeSocketsInBoneMap;

	if (BoneTransformStorageMode == EBoneTransformStorageMode::Direct)
	{
		InitForDirectStorage(bAllowScaling);
	}
	else if (bHasUnifiedBoneMap)
	{
		InitForBoneMapStorage();
	}
}

void FSkinningSceneExtensionProxy::InitForDirectStorage(bool bAllowScaling)
{
	check(BoneTransformStorageMode == EBoneTransformStorageMode::Direct);
	const FReferenceSkeleton& RefSkeleton = SkinnedAsset->GetRefSkeleton();
	const TArray<FTransform>& RefBonePose = RefSkeleton.GetRawRefBonePose();

	TArray<FTransform> ComponentTransforms;
	FAnimationRuntime::FillUpComponentSpaceTransforms(RefSkeleton, RefBonePose, ComponentTransforms);

	const uint16 MaxRawBoneCount = uint16(RefSkeleton.GetRawBoneNum());
	MaxBoneTransformCount = MaxRawBoneCount;

	BoneHierarchy.SetNumUninitialized(MaxRawBoneCount);

	bHasScale = false;

	const bool bRemoveScale = !bAllowScaling;

	for (int32 BoneIndex = 0; BoneIndex < MaxRawBoneCount; ++BoneIndex)
	{
		struct FPackedBone
		{
			uint32 BoneParent : 16;
			uint32 BoneDepth : 16;
		}
		Packed;

		const int32 ParentBoneIndex	= RefSkeleton.GetRawParentIndex(BoneIndex);
		const int32 BoneDepth		= RefSkeleton.GetDepthBetweenBones(BoneIndex, 0);
		Packed.BoneParent			= uint16(ParentBoneIndex);
		Packed.BoneDepth			= uint16(BoneDepth);
		BoneHierarchy[BoneIndex]	= *reinterpret_cast<uint32*>(&Packed);

		if (bRemoveScale)
		{
			ComponentTransforms[BoneIndex].RemoveScaling();
		}
		else if (!bHasScale && !FMath::IsNearlyEqual((float)ComponentTransforms[BoneIndex].GetDeterminant(), 1.0f, UE_KINDA_SMALL_NUMBER))
		{
			bHasScale = true;
		}
	}

	// TODO: Shrink/compress representation further
	// Drop one of the rotation components (largest value) and store index in 4 bits to reconstruct
	// 16b fixed point? Variable rate?
	const uint32 FloatCount = GetObjectSpaceFloatCount();
	BoneObjectSpace.SetNumUninitialized(MaxRawBoneCount * FloatCount);
	float* WritePtr = BoneObjectSpace.GetData();
	for (int32 BoneIndex = 0; BoneIndex < MaxRawBoneCount; ++BoneIndex)
	{
		const FTransform& Transform = ComponentTransforms[BoneIndex];
		const FQuat& Rotation = Transform.GetRotation();
		const FVector& Translation = Transform.GetTranslation();

		WritePtr[0] = (float)Rotation.X;
		WritePtr[1] = (float)Rotation.Y;
		WritePtr[2] = (float)Rotation.Z;
		WritePtr[3] = (float)Rotation.W;

		WritePtr[4] = (float)Translation.X;
		WritePtr[5] = (float)Translation.Y;
		WritePtr[6] = (float)Translation.Z;

		if (bHasScale)
		{
			const FVector& Scale = Transform.GetScale3D();
			WritePtr[7] = (float)Scale.X;
			WritePtr[8] = (float)Scale.Y;
			WritePtr[9] = (float)Scale.Z;
		}

		WritePtr += FloatCount;
	}
}

void FSkinningSceneExtensionProxy::GetSocketBoneIndices(const USkinnedAsset* InSkinnedAsset, TArray<FBoneIndexType>& OutBoneIndices)
{
	if (!InSkinnedAsset)
	{
		return;
	}

	const FReferenceSkeleton& RefSkeleton = InSkinnedAsset->GetRefSkeleton();
	const int32 NumRawBones = RefSkeleton.GetRawBoneNum();
	const TArray<USkeletalMeshSocket*> Sockets = InSkinnedAsset->GetActiveSocketList();

	for (const USkeletalMeshSocket* Socket : Sockets)
	{
		if (Socket)
		{
			const int32 BoneIndex = RefSkeleton.FindBoneIndex(Socket->BoneName);
			if (BoneIndex != INDEX_NONE && BoneIndex < NumRawBones)
			{
				OutBoneIndices.AddUnique(BoneIndex);
			}
		}
	}
}

void FSkinningSceneExtensionProxy::InitForBoneMapStorage()
{
	check(BoneTransformStorageMode == EBoneTransformStorageMode::BoneMap);
	TConstArrayView<FSkelMeshRenderSection> Sections = MeshObject->GetRenderSections(bHasUnifiedBoneMap ? MeshObject->MinLODLevel : LODIndex);
	check(!Sections.IsEmpty());

	MaxBoneTransformCount = 0;
	for (const FSkelMeshRenderSection& Section : Sections)
	{
		if (Section.IsValid())
		{
			MaxBoneTransformCount += Section.BoneMap.Num();

			// Only use the first found section in unified mode since they are all the same.
			if (bHasUnifiedBoneMap)
			{
				break;
			}
		}
	}
	BoneMap.Empty(MaxBoneTransformCount);

	for (const FSkelMeshRenderSection& Section : Sections)
	{
		if (Section.IsValid())
		{
			BoneMap.Append(Section.BoneMap);

			// Only use the first found section in unified mode since they are all the same.
			if (bHasUnifiedBoneMap)
			{
				break;
			}
		}
	}

	// Append socket bones so they have TransformIndex slots in the skinning buffer.
	if (bHasSocketsInBoneMap)
	{
		TArray<FBoneIndexType> SocketBones;
		GetSocketBoneIndices(SkinnedAsset, SocketBones);
		for (const FBoneIndexType BoneIndex : SocketBones)
		{
			BoneMap.AddUnique(BoneIndex);
		}
		MaxBoneTransformCount = BoneMap.Num();
	}

	check(MaxBoneTransformCount > 0);
}

const FGuid& FSkinningSceneExtensionProxy::GetTransformProviderId() const
{
	static FGuid MeshObjectProviderId(ANIM_MESH_OBJECT_TRANSFORM_PROVIDER_GUID);
	return MeshObjectProviderId;
}

FInstancedSkinningSceneExtensionProxy::FInstancedSkinningSceneExtensionProxy(
	const UTransformProviderData* TransformProvider,
	const FSkeletalMeshObject* InMeshObject,
	const USkinnedAsset* InSkinnedAsset,
	bool bAllowScaling,
	bool bInIncludeSocketsInBoneMap)
	: FSkinningSceneExtensionProxy(InMeshObject, InSkinnedAsset, bAllowScaling, bInIncludeSocketsInBoneMap)
{
	bUseInstancing = true;

	bool bSucceeded = false;

	const bool bForceRefPose = CVarInstancedSkinnedMeshesForceRefPose.GetValueOnAnyThread() != 0;
	if (!bForceRefPose && GetSkinnedAsset() && TransformProvider != nullptr && TransformProvider->IsEnabled())
	{
		TransformProviderProxy = TransformProvider->CreateRenderProxy(this);

		if (TransformProviderProxy)
		{
			TransformProviderId    = TransformProvider->GetTransformProviderID();
			bUseSkeletonBatching   = TransformProvider->UsesSkeletonBatching();
			SetUniqueAnimationCount(TransformProvider->GetUniqueAnimationCount());
			bSucceeded             = true;
		}
	}

	if (!bSucceeded)
	{
		UniqueAnimationCount = 1;
		bUseSkeletonBatching = false;

		static FGuid RefPoseProviderId(REF_POSE_TRANSFORM_PROVIDER_GUID);
		TransformProviderId = RefPoseProviderId;
	}
}

void FInstancedSkinningSceneExtensionProxy::CreateRenderThreadResources(FSceneInterface& Scene, FRHICommandListBase& RHICmdList)
{
	if (TransformProviderProxy)
	{
		TransformProviderProxy->CreateRenderThreadResources(Scene, RHICmdList);
	}
}

void FInstancedSkinningSceneExtensionProxy::DestroyRenderThreadResources()
{
	if (TransformProviderProxy)
	{
		TransformProviderProxy->DestroyRenderThreadResources();
		delete TransformProviderProxy;
		TransformProviderProxy = nullptr;
	}
}

const FGuid& FInstancedSkinningSceneExtensionProxy::GetTransformProviderId() const
{
	return TransformProviderId;
}
