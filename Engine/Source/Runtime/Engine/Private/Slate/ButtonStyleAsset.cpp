// Copyright Epic Games, Inc. All Rights Reserved.

#include "Slate/ButtonStyleAsset.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ButtonStyleAsset)

UButtonStyleAsset::UButtonStyleAsset( const FObjectInitializer& ObjectInitializer )
	: Super(ObjectInitializer)
{
	
}

