// Copyright Epic Games, Inc. All Rights Reserved.

#include "Slate/CheckboxStyleAsset.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CheckboxStyleAsset)

UCheckBoxStyleAsset::UCheckBoxStyleAsset( const FObjectInitializer& ObjectInitializer )
	: Super(ObjectInitializer)
{
	
}

