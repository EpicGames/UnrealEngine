// Copyright Epic Games, Inc. All Rights Reserved.

#include "Slate/DebugCanvas.h"
#include "Engine/World.h"
#include "RenderingThread.h"
#include "CanvasTypes.h"
#include "Engine/Engine.h"
#include "EngineFontServices.h"
#include "Framework/Application/SlateApplication.h"
#include "IStereoLayers.h"
#include "ProfilingDebugging/RealtimeGPUProfiler.h"
#include "StereoRendering.h"
#include "Slate/SceneViewport.h"
#include "IXRTrackingSystem.h"
#include "ISpectatorScreenController.h"
#include "IHeadMountedDisplay.h"
#include "RenderTargetPool.h"
#include "ViewportClient.h"
#include "RenderGraphUtils.h"
#include "Engine/TextureRenderTarget2D.h"
#include "TextureResource.h"

/**
 * Simple representation of the backbuffer that the debug canvas renders to
 * This class may only be accessed from the render thread
 */
class FSlateCanvasRenderTarget : public FRenderTarget
{
public:
	/** FRenderTarget interface */
	virtual FIntPoint GetSizeXY() const
	{
		if (IsInRenderingThread())
		{
			return ViewRect_RenderThread.Size();
		}
		else
		{
			return ViewRect_GameThread.Size();
		}
	}

	/** Sets the texture that this target renders to */
	void SetRenderTargetTexture(FRDGTexture* InRDGTexture)
	{
		RDGTexture = InRDGTexture;
	}

	/** Clears the render target texture */
	void ClearRenderTargetTexture()
	{
		RDGTexture = nullptr;
	}

	const FTextureRHIRef& GetRenderTargetTexture() const override
	{
		static FTextureRHIRef NullRef;
		return NullRef;
	}

	FRDGTextureRef GetRenderTargetTexture(FRDGBuilder&) const override
	{
		return RDGTexture;
	}

	void SetViewRect_GameThread(const FIntRect& InViewRect)
	{
		ViewRect_GameThread = InViewRect;
	}

	void SetViewRect_RenderThread(const FIntRect& InViewRect)
	{
		ViewRect_RenderThread = InViewRect;
	}

	/** Gets the viewport rect for the render target */
	const FIntRect& GetViewRect_GameThread() const 
	{
		return ViewRect_GameThread;
	}

	const FIntRect& GetViewRect_RenderThread() const 
	{
		return ViewRect_RenderThread;
	}

private:
	FRDGTexture* RDGTexture = nullptr;
	FIntRect ViewRect_GameThread;
	FIntRect ViewRect_RenderThread;
};

#define INVALID_LAYER_ID UINT_MAX

FDebugCanvasDrawer::FDebugCanvasDrawer()
	: GameThreadCanvas( NULL )
	, RenderThreadCanvas( NULL )
	, RenderTarget( new FSlateCanvasRenderTarget )
	, LayerID(INVALID_LAYER_ID)
{
	// watch for font cache flushes
	if (FEngineFontServices::IsInitialized())
	{
		FEngineFontServices::Get().OnReleaseResources().AddRaw(this, &FDebugCanvasDrawer::HandleReleaseFontResources);
	}
}

void FDebugCanvasDrawer::ReleaseInternalTexture()
{
	check(IsInGameThread());

	if (IsValid(LayerTexture))
	{
		LayerTexture->ReleaseResource();
		LayerTexture = nullptr;
	}

	if (LayerID != INVALID_LAYER_ID
		&& GEngine
    	&& GEngine->StereoRenderingDevice.IsValid()
    	&& GEngine->StereoRenderingDevice->GetStereoLayers())
    {
    	GEngine->StereoRenderingDevice->GetStereoLayers()->DestroyLayer(LayerID);
    	LayerID = INVALID_LAYER_ID;
    }
}

void FDebugCanvasDrawer::HandleReleaseFontResources(const class FSlateFontCache& InFontCache)
{
	check(IsInGameThread());

	// If this function is called while we have a pending render Canvas request, then we need to force 
	// a flush on the render thread to clear the pending batches that may reference invalid resources
	if (RenderThreadCanvas)
	{
		ENQUEUE_RENDER_COMMAND(FlushFontResourcesCommand)(
			[this](FRHICommandListImmediate& RHICmdList)
		{
			RenderThreadCanvas->Flush_RenderThread(RHICmdList, true);
		});

		FlushRenderingCommands();
	}

	// If this function is called while the game thread is still prepping a Canvas, then we need to 
	// force clear the pending batches as they may reference invalid resources
	if (GameThreadCanvas)
	{
		GameThreadCanvas->ClearBatchesToRender();
	}
}

void FDebugCanvasDrawer::AddReferencedObjects(FReferenceCollector& Collector)
{
	Collector.AddReferencedObject(LayerTexture);
}

FDebugCanvasDrawer::~FDebugCanvasDrawer()
{
	// stop watching for font cache flushes
	if (FEngineFontServices::IsInitialized())
	{
		FEngineFontServices::Get().OnReleaseResources().RemoveAll(this);
	}

	delete RenderTarget;

	ReleaseInternalTexture();

	// We assume that the render thread is no longer utilizing any canvases
	if( GameThreadCanvas.IsValid() && RenderThreadCanvas != GameThreadCanvas )
	{
		GameThreadCanvas.Reset();
	}

	if( RenderThreadCanvas.IsValid() )
	{
		// Capture a copy of the canvas until the render thread can delete it
		FCanvasPtr RTCanvas = RenderThreadCanvas;
		ENQUEUE_RENDER_COMMAND(DeleteDebugRenderThreadCanvas)(
			[RTCanvas](FRHICommandListImmediate& RHICmdList)
		{
		});

		RenderThreadCanvas = nullptr;
	}
}

FCanvas* FDebugCanvasDrawer::GetGameThreadDebugCanvas()
{
	return GameThreadCanvas.Get();
}

void FDebugCanvasDrawer::BeginRenderingCanvas( const FIntRect& CanvasRect )
{
	if( CanvasRect.Size().X > 0 && CanvasRect.Size().Y > 0 )
	{
		bCanvasRenderedLastFrame = true;
		FDebugCanvasDrawer* CanvasDrawer = this;
		FCanvasPtr CanvasToRender = GameThreadCanvas;
		FTextureRenderTargetResource* XLayerResource = nullptr;
		if (LayerTexture)
		{
			LayerTexture->ResizeTarget(CanvasRect.Width(), CanvasRect.Height());
			XLayerResource = LayerTexture->GameThread_GetRenderTargetResource();
		}

		// Caching the ViewRect for the game thread before passing the canvas to the render thread
		if (CanvasToRender.IsValid() && CanvasToRender->IsUsingInternalTexture())
		{
			RenderTarget->SetViewRect_GameThread(FIntRect(FIntPoint(0, 0), CanvasToRender->GetParentCanvasSize()));
		}
		else
		{
			RenderTarget->SetViewRect_GameThread(CanvasRect);
		}

		ENQUEUE_RENDER_COMMAND(BeginRenderingDebugCanvas)(
			[CanvasDrawer, CanvasToRender, XLayerResource, CanvasRect = CanvasRect](FRHICommandListImmediate& RHICmdList)
			{
				FCanvasPtr LocalCanvasToRender = CanvasToRender;
			
				// Delete the old rendering thread canvas
				if( CanvasDrawer->GetRenderThreadCanvas().IsValid() && LocalCanvasToRender.IsValid() )
				{
					CanvasDrawer->DeleteRenderThreadCanvas();
				}

				if (!LocalCanvasToRender.IsValid())
				{
					LocalCanvasToRender = CanvasDrawer->GetRenderThreadCanvas();
				}

				CanvasDrawer->SetRenderThreadCanvas( CanvasRect, LocalCanvasToRender, XLayerResource );
			}
		);
		
		// Gave the canvas to the render thread
		GameThreadCanvas = nullptr;
	}
}


void FDebugCanvasDrawer::InitDebugCanvas(FViewportClient* ViewportClient, UWorld* InWorld)
{
	const bool bIsStereoscopic3D = GEngine && GEngine->IsStereoscopic3D();
	IStereoLayers* const StereoLayers = (bIsStereoscopic3D && GEngine && GEngine->StereoRenderingDevice.IsValid()) ? GEngine->StereoRenderingDevice->GetStereoLayers() : nullptr;
	const bool bUseInternalTexture = StereoLayers && bIsStereoscopic3D;

	// If the canvas is not null there is more than one viewport draw call before slate draws.  This can happen on resizes. 
	// We need to delete the old canvas
		// This can also happen if we are debugging a HUD blueprint and in that case we need to continue using
		// the same canvas
	if (FSlateApplication::Get().IsNormalExecution())
	{
		const float DPIScale = bUseInternalTexture ? 1.0f : ViewportClient->GetDPIScale();
		GameThreadCanvas = MakeShared<FCanvas, ESPMode::ThreadSafe>(RenderTarget, nullptr, InWorld, InWorld ? InWorld->GetFeatureLevel() : GMaxRHIFeatureLevel, FCanvas::CDM_DeferDrawing, DPIScale);

		// Do not allow the canvas to be flushed outside of our debug rendering path
		GameThreadCanvas->SetAllowedModes(FCanvas::Allow_DeleteOnRender);
	}

	if (GameThreadCanvas.IsValid())
	{
		GameThreadCanvas->SetUseInternalTexture(bUseInternalTexture);

		if (bUseInternalTexture)
		{
			if (!IsValid(LayerTexture) && bCanvasRenderedLastFrame)
			{
				LayerTexture = NewObject<UTextureRenderTarget2D>();
				LayerTexture->ClearColor = FLinearColor::Transparent;
			}

			if (StereoLayers && (bCanvasRenderedLastFrame || LayerID != INVALID_LAYER_ID))
			{
				IStereoLayers::FLayerDesc StereoLayerDesc = StereoLayers->GetDebugCanvasLayerDesc(LayerTexture);
				StereoLayerDesc.Flags |= !bCanvasRenderedLastFrame ? IStereoLayers::LAYER_FLAG_HIDDEN : 0;
				StereoLayerDesc.Flags |= IStereoLayers::LAYER_FLAG_DEBUG;

				if (LayerID == INVALID_LAYER_ID && bCanvasRenderedLastFrame)
				{
					LayerID = StereoLayers->CreateLayer(StereoLayerDesc);
				}
				else if (LayerID != INVALID_LAYER_ID)
				{
					StereoLayers->SetLayerDesc(LayerID, StereoLayerDesc);
				}
			}
		}
		else
		{
			ReleaseInternalTexture();
		}

		bCanvasRenderedLastFrame = false;
	}
}

void FDebugCanvasDrawer::Draw_RenderThread(FRDGBuilder& GraphBuilder, const FDrawPassInputs& Inputs)
{
	RDG_EVENT_SCOPE(GraphBuilder, "DrawDebugCanvas");
	TRACE_CPUPROFILER_EVENT_SCOPE(DrawDebugCanvas);

	if (RenderThreadCanvas.IsValid())
	{
		FRDGTexture* OutputTexture = Inputs.OutputTexture;

		if (RenderThreadCanvas->IsUsingInternalTexture())
		{
			if (LayerResource && LayerResource->GetRenderTargetTexture())
			{
				OutputTexture = RegisterExternalTexture(GraphBuilder, LayerResource->GetRenderTargetTexture(), TEXT("HMDDebugLayerTexture"));
				AddClearRenderTargetPass(GraphBuilder, OutputTexture);
			}
			else
			{
				UE_LOGF(LogProfilingDebugging, Warning, "No layer resource or HMD swapchain available for stereo debug layer");
			}
		}

		RenderTarget->SetRenderTargetTexture(OutputTexture);

		if (RenderThreadCanvas->IsUsingInternalTexture())
		{
			RenderThreadCanvas->SetRenderTargetRect(FIntRect(FIntPoint::ZeroValue, OutputTexture->Desc.Extent));
		}
		else
		{
			RenderThreadCanvas->SetRenderTargetRect(RenderTarget->GetViewRect_RenderThread());
		}

		RenderThreadCanvas->Flush_RenderThread(GraphBuilder, true);

		RenderTarget->ClearRenderTargetTexture();
	}
}

FCanvasPtr FDebugCanvasDrawer::GetRenderThreadCanvas()
{
	check( IsInRenderingThread() );
	return RenderThreadCanvas;
}

void FDebugCanvasDrawer::DeleteRenderThreadCanvas()
{
	check( IsInRenderingThread() );
	RenderThreadCanvas.Reset();
}

void FDebugCanvasDrawer::SetRenderThreadCanvas( const FIntRect& InCanvasRect, FCanvasPtr& Canvas, FTextureRenderTargetResource* InLayerResource )
{
	check( IsInRenderingThread() );
	if (Canvas->IsUsingInternalTexture())
	{
		RenderTarget->SetViewRect_RenderThread(FIntRect(FIntPoint(0, 0), Canvas->GetParentCanvasSize()));
	}
	else
	{
		RenderTarget->SetViewRect_RenderThread(InCanvasRect);
	}
	RenderThreadCanvas = Canvas;
	LayerResource = InLayerResource;
}

SDebugCanvas::SDebugCanvas()
{
	SetCanTick(false);
	bCanSupportFocus = false;
}

void SDebugCanvas::Construct(const FArguments& InArgs)
{
	SceneViewport = InArgs._SceneViewport;
	bHasCustomPrepass = true;
}

bool SDebugCanvas::CustomPrepass(float LayoutScaleMultiplier)
{
	SetIsProjectContent(false);
	return false;
}

int32 SDebugCanvas::OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_SlatePaintDebugCanvas);
	const FSceneViewport* Viewport = SceneViewport.Get();
	if (Viewport)
	{
		Viewport->PaintDebugCanvas(AllottedGeometry, OutDrawElements, LayerId);
	}

	return LayerId;
}

FVector2D SDebugCanvas::ComputeDesiredSize(float LayoutScaleMultiplier) const
{
	const FSceneViewport* Viewport = SceneViewport.Get();
	if (Viewport)
	{
		return Viewport->GetSizeXY();
	}
	else
	{
		return FVector2D::ZeroVector;
	}
}

void SDebugCanvas::SetSceneViewport(FSceneViewport* InSceneViewport)
{
	FSceneViewport* CurrentSceneViewport = SceneViewport.Get();
	if (CurrentSceneViewport)
	{
		// this canvas is moving to another viewport
		CurrentSceneViewport->SetDebugCanvas(nullptr);
	}

	SceneViewport = InSceneViewport;

	if (InSceneViewport)
	{
		// Notify the new viewport of its debug canvas for invalidation purposes
		InSceneViewport->SetDebugCanvas(SharedThis(this));
	}
}
