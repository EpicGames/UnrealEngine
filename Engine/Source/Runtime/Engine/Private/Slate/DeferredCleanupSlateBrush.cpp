// Copyright Epic Games, Inc. All Rights Reserved.

#include "Slate/DeferredCleanupSlateBrush.h"
#include "Engine/Texture.h"

TSharedRef<FDeferredCleanupSlateBrush> FDeferredCleanupSlateBrush::CreateBrush(const FSlateBrush& Brush)
{
	return MakeShareable(new FDeferredCleanupSlateBrush(Brush), [](FDeferredCleanupSlateBrush* ObjectToDelete) { BeginCleanup(ObjectToDelete); });
}

TSharedRef<FDeferredCleanupSlateBrush> FDeferredCleanupSlateBrush::CreateBrush(
	UTexture* InTexture,
	const FLinearColor& InTint,
	ESlateBrushTileType::Type InTiling,
	ESlateBrushImageType::Type InImageType,
	ESlateBrushDrawType::Type InDrawType,
	const FMargin& InMargin)
{
	FSlateBrush Brush;
	Brush.SetResourceObject(InTexture);
	Brush.ImageSize = FVector2f(InTexture->GetSurfaceWidth(), InTexture->GetSurfaceHeight());
	Brush.TintColor = InTint;
	Brush.Margin = InMargin;
	Brush.Tiling = InTiling;
	Brush.ImageType = InImageType;
	Brush.DrawAs = InDrawType;

	return MakeShareable(new FDeferredCleanupSlateBrush(Brush), [](FDeferredCleanupSlateBrush* ObjectToDelete) { BeginCleanup(ObjectToDelete); });
}

TSharedRef<FDeferredCleanupSlateBrush> FDeferredCleanupSlateBrush::CreateBrush(
	UObject* InResource,
	const UE::Slate::FDeprecateVector2DParameter& InImageSize,
	const FLinearColor& InTint,
	ESlateBrushTileType::Type InTiling,
	ESlateBrushImageType::Type InImageType,
	ESlateBrushDrawType::Type InDrawType,
	const FMargin& InMargin)
{
	FSlateBrush Brush;
	Brush.SetResourceObject(InResource);
	Brush.ImageSize = InImageSize;
	Brush.TintColor = InTint;
	Brush.Margin = InMargin;
	Brush.Tiling = InTiling;
	Brush.ImageType = InImageType;
	Brush.DrawAs = InDrawType;

	return MakeShareable(new FDeferredCleanupSlateBrush(Brush), [](FDeferredCleanupSlateBrush* ObjectToDelete) { BeginCleanup(ObjectToDelete); });
}

void FDeferredCleanupSlateBrush::AddReferencedObjects(FReferenceCollector& Collector)
{
	InternalBrush.AddReferencedObjects(Collector);
}

FString FDeferredCleanupSlateBrush::GetReferencerName() const
{
	return TEXT("FDeferredCleanupSlateBrush");
}