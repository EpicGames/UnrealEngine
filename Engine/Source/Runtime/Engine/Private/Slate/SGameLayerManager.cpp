// Copyright Epic Games, Inc. All Rights Reserved.

#include "Slate/SGameLayerManager.h"
#include "Engine/GameViewportClient.h"
#include "Engine/LocalPlayer.h"
#include "Engine/World.h"
#include "Slate/SceneViewport.h"
#include "SceneView.h"
#include "Types/NavigationMetaData.h"
#include "Engine/GameEngine.h"
#include "Engine/UserInterfaceSettings.h"
#include "GeneralProjectSettings.h"
#include "GameMapsSettings.h"
#include "Input/HittestGrid.h"
#include "Widgets/LayerManager/STooltipPresenter.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SDPIScaler.h"
#include "Widgets/Layout/SPopup.h"
#include "Widgets/Layout/SWindowTitleBarArea.h"
#include "Slate/DebugCanvas.h"
#include "Types/InvisibleToWidgetReflectorMetaData.h"
#include "Framework/Application/SlateApplication.h"

#if WITH_SLATE_NAVIGATION_DEBUGGING
#include "Navigation/SlateNavigationDebugger.h"
#endif //WITH_SLATE_NAVIGATION_DEBUGGING

#include UE_INLINE_GENERATED_CPP_BY_NAME(SGameLayerManager)

#ifndef UE_SLATE_WITH_GAMELAYER_CANVAS_VISIBILITY_COMMANDS
	#define UE_SLATE_WITH_GAMELAYER_CANVAS_VISIBILITY_COMMANDS !(UE_BUILD_SHIPPING || UE_BUILD_TEST) && WITH_SLATE_DEBUGGING
#endif

/* SGameLayerManager interface
 *****************************************************************************/
static void HandlePerUserHitTestingToggled(IConsoleVariable* CVar)
{
	FSlateApplication::Get().InvalidateAllWidgets(false);
}

static int32 bEnablePerUserHitTesting = true;

static FAutoConsoleVariableRef CVarEnablePerUserHitTesting(
	TEXT("Slate.AllowPerUserHitTesting"),
	bEnablePerUserHitTesting,
	TEXT("Toggles between widgets mapping to a user id and requring a matching user id from an input event or allowing all users to interact with widget"),
	FConsoleVariableDelegate::CreateStatic(&HandlePerUserHitTestingToggled)
);

#if UE_SLATE_WITH_GAMELAYER_CANVAS_VISIBILITY_COMMANDS
namespace CanvasVisibility
{
	static TArray<SGameLayerManager*> GameLayerManagerInstances;
	static bool ViewportSlotVisibilityConsoleValue = true;
	static bool DebugCanvasVisibilityConsoleValue = true;
	static bool PlayerCanvasVisibilityConsoleValue = true;
	static bool CanvasVisibilityCVarOnChangedBound = false;

	enum class CanvasType
	{
		PlayerCanvas,
		DebugCanvas,
		Viewport,
		AllCanvases
	};
}

static FAutoConsoleVariableRef CVarShowViewportSlot(
	TEXT("Slate.GameLayer.ViewportSlotVisible"),
	CanvasVisibility::ViewportSlotVisibilityConsoleValue,
	TEXT("Show/Hide the slot on viewport"),
	ECVF_Cheat
);
static FAutoConsoleVariableRef CVarShowDebugCanvas(
	TEXT("Slate.GameLayer.DebugCanvasVisible"),
	CanvasVisibility::DebugCanvasVisibilityConsoleValue,
	TEXT("Show/Hide the debug canvas."),
	ECVF_Cheat
);
static FAutoConsoleVariableRef CVarShowPlayerCanvas(
	TEXT("Slate.GameLayer.PlayerCanvasVisible"),
	CanvasVisibility::PlayerCanvasVisibilityConsoleValue,
	TEXT("Show/Hide the player canvas."),
	ECVF_Cheat
);
static TAutoConsoleVariable<bool> CVarShowAllCanvases(
	TEXT("Slate.GameLayer.AllCanvasesVisible"),
	true,
	TEXT("Show/Hide the viewport slot, player canvas, and debug canvas."),
	ECVF_Cheat
);
#endif

SGameLayerManager::SGameLayerManager()
:	DefaultWindowTitleBarHeight(64.0f)
,	bIsGameUsingBorderlessWindow(false)
,	bUseScaledDPI(false)
,	CachedInverseDPIScale(0.0f)
{
}

void SGameLayerManager::Construct(const SGameLayerManager::FArguments& InArgs)
{
	bHasCustomPrepass = true;
	SceneViewport = InArgs._SceneViewport;

	// In PIE we should default to per user hit testing being off because developers will need the mouse and keyboard to work for all players
	if (GIsEditor)
	{
		bEnablePerUserHitTesting = false;
	}

	TSharedRef<SDPIScaler> DPIScaler =
		SNew(SDPIScaler)
		.DPIScale(this, &SGameLayerManager::GetGameViewportDPIScale)
		[
			// All user widgets live inside this vertical box.
			SAssignNew(WidgetHost, SVerticalBox)

			+ SVerticalBox::Slot()
			.AutoHeight()
			[
				SAssignNew(TitleBarAreaVerticalBox, SWindowTitleBarArea)
				[
					SAssignNew(WindowTitleBarVerticalBox, SBox)
				]
			]

			+ SVerticalBox::Slot()
			[
				SAssignNew(WindowOverlay, SOverlay)

				+ SOverlay::Slot(static_cast<int32>(EGameLayerOrder::Player))
				[
					SAssignNew(PlayerCanvas, SCanvas)
				]

				+ SOverlay::Slot(static_cast<int32>(EGameLayerOrder::Viewport))
				[
					SAssignNew(ViewportSlotContainer, SBox)
					[
						InArgs._Content.Widget
					]
				]

				+ SOverlay::Slot(static_cast<int32>(EGameLayerOrder::TitleBar))
				[
					SNew(SVerticalBox)

					+ SVerticalBox::Slot()
					.AutoHeight()
					[
						SAssignNew(TitleBarAreaOverlay, SWindowTitleBarArea)
						[
							SAssignNew(WindowTitleBarOverlay, SBox)
						]
					]
				]

				+ SOverlay::Slot(static_cast<int32>(EGameLayerOrder::Tooltip))
				[
					SNew(SPopup)
					[
						SAssignNew(TooltipPresenter, STooltipPresenter)
					]
				]
				+ SOverlay::Slot(static_cast<int32>(EGameLayerOrder::Debug))
				[
					SAssignNew(DebugCanvas, SDebugCanvas)
					.SceneViewport(InArgs._SceneViewport)
					.AddMetaData<FInvisibleToWidgetReflectorMetaData>(FInvisibleToWidgetReflectorMetaData())
				]
			]
		];

	ChildSlot
	[
		DPIScaler
	];

	UGameEngine* GameEngine = Cast<UGameEngine>(GEngine);
	if (GameEngine != nullptr)
	{
		TSharedPtr<SWindow> GameViewportWindow = GameEngine->GameViewportWindow.Pin();
		if (GameViewportWindow.IsValid())
		{
			TitleBarAreaOverlay->SetGameWindow(GameViewportWindow);
			TitleBarAreaVerticalBox->SetGameWindow(GameViewportWindow);
		}
	}

	DefaultTitleBarContentWidget =
		SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew(SBox).HeightOverride(this, &SGameLayerManager::GetDefaultWindowTitleBarHeight)
		];

	TitleBarAreaOverlay->SetRequestToggleFullscreenCallback(FSimpleDelegate::CreateSP(this, &SGameLayerManager::RequestToggleFullscreen));
	TitleBarAreaVerticalBox->SetRequestToggleFullscreenCallback(FSimpleDelegate::CreateSP(this, &SGameLayerManager::RequestToggleFullscreen));

	SetWindowTitleBarState(nullptr, EWindowTitleBarMode::Overlay, false, false, false);
	
	bIsGameUsingBorderlessWindow = GetDefault<UGeneralProjectSettings>()->bUseBorderlessWindow && PLATFORM_WINDOWS;

#if UE_SLATE_WITH_GAMELAYER_CANVAS_VISIBILITY_COMMANDS
	CanvasVisibility::GameLayerManagerInstances.Add(this);
	if (!CanvasVisibility::CanvasVisibilityCVarOnChangedBound)
	{
		auto ShowHUD = [](IConsoleVariable* Variable, CanvasVisibility::CanvasType CanvasType)
		{
			bool bVisible = Variable->GetBool();
			for (int32 Index = 0; Index < CanvasVisibility::GameLayerManagerInstances.Num(); ++Index)
			{
				if (SGameLayerManager* GameManagerInstance = CanvasVisibility::GameLayerManagerInstances[Index])
				{
					switch (CanvasType)
					{
					case CanvasVisibility::CanvasType::PlayerCanvas:
						GameManagerInstance->ShowPlayerCanvas(bVisible);
						break;
					case CanvasVisibility::CanvasType::DebugCanvas:
						GameManagerInstance->ShowDebugCanvas(bVisible);
						break;
					case CanvasVisibility::CanvasType::Viewport:
						GameManagerInstance->ShowViewportSlot(bVisible);
						break;
					case CanvasVisibility::CanvasType::AllCanvases:
						GameManagerInstance->ShowViewportSlot(bVisible);
						GameManagerInstance->ShowDebugCanvas(bVisible);
						GameManagerInstance->ShowPlayerCanvas(bVisible);
						CanvasVisibility::ViewportSlotVisibilityConsoleValue = bVisible;
						CanvasVisibility::PlayerCanvasVisibilityConsoleValue = bVisible;
						CanvasVisibility::DebugCanvasVisibilityConsoleValue = bVisible;
						break;
					default:
						break;
					}
				}
			};
		};
		CVarShowAllCanvases->AsVariable()->SetOnChangedCallback(FConsoleVariableDelegate::CreateLambda(ShowHUD, CanvasVisibility::CanvasType::AllCanvases));
		CVarShowPlayerCanvas->AsVariable()->SetOnChangedCallback(FConsoleVariableDelegate::CreateLambda(ShowHUD, CanvasVisibility::CanvasType::PlayerCanvas));
		CVarShowDebugCanvas->AsVariable()->SetOnChangedCallback(FConsoleVariableDelegate::CreateLambda(ShowHUD, CanvasVisibility::CanvasType::DebugCanvas));
		CVarShowViewportSlot->AsVariable()->SetOnChangedCallback(FConsoleVariableDelegate::CreateLambda(ShowHUD, CanvasVisibility::CanvasType::Viewport));
		CanvasVisibility::CanvasVisibilityCVarOnChangedBound = true;
	}

	ShowViewportSlot(CanvasVisibility::ViewportSlotVisibilityConsoleValue);
	ShowDebugCanvas(CanvasVisibility::DebugCanvasVisibilityConsoleValue);
	ShowPlayerCanvas(CanvasVisibility::PlayerCanvasVisibilityConsoleValue);
#endif
}

void SGameLayerManager::SetSceneViewport(FSceneViewport* InSceneViewport)
{
	SceneViewport = InSceneViewport;
	DebugCanvas->SetSceneViewport(InSceneViewport);
}

FGeometry SGameLayerManager::GetViewportWidgetHostGeometry() const
{
	return WidgetHost->GetTickSpaceGeometry();
}

FGeometry SGameLayerManager::GetViewportWidgetHostPaintGeometry() const
{
	return WidgetHost->GetPaintSpaceGeometry();
}

FGeometry SGameLayerManager::GetPlayerWidgetHostGeometry(ULocalPlayer* Player) const
{
	TSharedPtr<FPlayerLayer> PlayerLayer = PlayerLayers.FindRef(Player);
	if ( PlayerLayer.IsValid() )
	{
		return PlayerLayer->Widget->GetTickSpaceGeometry();
	}

	static FGeometry Identity;
	return Identity;
}

void SGameLayerManager::NotifyPlayerAdded(int32 PlayerIndex, ULocalPlayer* AddedPlayer)
{
	UpdateLayout();
}

void SGameLayerManager::NotifyPlayerRemoved(int32 PlayerIndex, ULocalPlayer* RemovedPlayer)
{
	UpdateLayout();
}

void SGameLayerManager::AddWidgetForPlayer(ULocalPlayer* Player, TSharedRef<SWidget> ViewportContent, const int32 ZOrder)
{
	TSharedPtr<FPlayerLayer> PlayerLayer = FindOrCreatePlayerLayer(Player);

	PlayerLayer->Widget->AddSlot(ZOrder)
	[
		ViewportContent
	];
}

void SGameLayerManager::RemoveWidgetForPlayer(ULocalPlayer* Player, TSharedRef<SWidget> ViewportContent)
{
	if (TSharedPtr<FPlayerLayer> PlayerLayer = PlayerLayers.FindRef(Player) )
	{
		PlayerLayer->Widget->RemoveSlot(ViewportContent);
	}
	else
	{
		// If no local user is specified, we need to find the widget and purge it.
		for (auto& PlayerLayerEntry : PlayerLayers)
		{
			const TSharedPtr<FPlayerLayer>& PlayerLayerRef = PlayerLayerEntry.Value;
			if (PlayerLayerRef->Widget->RemoveSlot(ViewportContent))
			{
				return;
			}
		}
	}
}

void SGameLayerManager::ClearWidgetsForPlayer(ULocalPlayer* Player)
{
	TSharedPtr<FPlayerLayer> PlayerLayer = PlayerLayers.FindRef(Player);
	if ( PlayerLayer.IsValid() )
	{
		PlayerLayer->Widget->ClearChildren();
	}
}

TSharedPtr<IGameLayer> SGameLayerManager::FindLayerForPlayer(ULocalPlayer* Player, const FName& LayerName)
{
	TSharedPtr<FPlayerLayer> PlayerLayer = PlayerLayers.FindRef(Player);
	if ( PlayerLayer.IsValid() )
	{
		return PlayerLayer->Layers.FindRef(LayerName);
	}

	return TSharedPtr<IGameLayer>();
}

bool SGameLayerManager::AddLayerForPlayer(ULocalPlayer* Player, const FName& LayerName, TSharedRef<IGameLayer> Layer, int32 ZOrder)
{
	TSharedPtr<FPlayerLayer> PlayerLayer = FindOrCreatePlayerLayer(Player);
	if ( PlayerLayer.IsValid() )
	{
		TSharedPtr<IGameLayer> ExistingLayer = PlayerLayer->Layers.FindRef(LayerName);
		if ( ExistingLayer.IsValid() )
		{
			return false;
		}

		PlayerLayer->Layers.Add(LayerName, Layer);

		PlayerLayer->Widget->AddSlot(ZOrder)
		[
			Layer->AsWidget()
		];

		return true;
	}

	return false;
}

void SGameLayerManager::ClearWidgets()
{
	PlayerCanvas->ClearChildren();

	for (const TWeakPtr<SWidget>& GameLayerWeak : GameLayers)
	{
		if (const TSharedPtr<SWidget>& GameLayer = GameLayerWeak.Pin())
		{
			WindowOverlay->RemoveSlot(GameLayer.ToSharedRef());
		}
	}

	GameLayers.Empty();

	// Potential for removed layers to impact the map, so need to
	// remove & delete as separate steps
	while (PlayerLayers.Num())
	{
		const auto LayerIt = PlayerLayers.CreateIterator();
		const TSharedPtr<FPlayerLayer> Layer = LayerIt.Value();

		if (Layer.IsValid())
		{
			Layer->Slot = nullptr;
		}

		PlayerLayers.Remove(LayerIt.Key());
	}

	SetWindowTitleBarState(nullptr, EWindowTitleBarMode::Overlay, false, false, false);
}

void SGameLayerManager::AddGameLayer(TSharedRef<SWidget> ViewportContent, int32 ZOrder)
{
	GameLayers.Add(ViewportContent);

	ensure(!WindowOverlay->HasSlotWithZOrder(ZOrder));

	WindowOverlay->AddSlot(ZOrder)
	[
		ViewportContent
	];
}

void SGameLayerManager::RemoveGameLayer(TSharedRef<SWidget> ViewportContent)
{
	WindowOverlay->RemoveSlot(ViewportContent);
}

void SGameLayerManager::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	//if (AllottedGeometry != CachedGeometry)
	{
		CachedGeometry = AllottedGeometry;
		UpdateLayout();
	}
}

int32 SGameLayerManager::OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const
{
	SCOPED_NAMED_EVENT_TEXT("Paint: Game UI", FColor::Green);
#if WITH_EDITOR
	if (GIntraFrameDebuggingGameThread)
	{
		// When BP debugging, do not paint the PIE game.
		//It may trigger other BP code and while it's in a Debugging state the BP will not execute.
		//It may also tick widgets that may already be ticking.
		// ie. [A] SMyWidget::Tick() => BPEvent => BP Breakpoint => FApplication::EnterDebuggingMode() => while( SWindow::Paint() => SMyWidget::Tick() ) => End of the first [A] SMyWidget::Tick
		return FMath::Max(GetPersistentState().OutgoingLayerId, LayerId);
	}
#endif

	int32 ResultLayer = SCompoundWidget::OnPaint(Args, AllottedGeometry, MyCullingRect, OutDrawElements, LayerId, InWidgetStyle, bParentEnabled);

#if WITH_SLATE_NAVIGATION_DEBUGGING
	ResultLayer = FSlateNavigationDebugger::Get().Paint(OutDrawElements, ResultLayer);
#endif

	return ResultLayer;
}

bool SGameLayerManager::OnVisualizeTooltip(const TSharedPtr<SWidget>& TooltipContent)
{
	TooltipPresenter->SetContent(TooltipContent);

	return true;
}

void SGameLayerManager::SetUseFixedDPIValue(const bool bInUseFixedDPI, const FIntPoint ViewportSize /*= FIntPoint()*/)
{
	bUseScaledDPI = bInUseFixedDPI;
	ScaledDPIViewportReference = ViewportSize;
}

bool SGameLayerManager::IsUsingFixedDPIValue() const
{
	return bUseScaledDPI;
}

float SGameLayerManager::GetGameViewportDPIScale() const
{
	const FSceneViewport* Viewport = SceneViewport.Get();

	if (Viewport == nullptr)
	{
		return 1;
	}

	const UUserInterfaceSettings* UserInterfaceSettings = GetDefault<UUserInterfaceSettings>();
	const FIntPoint ViewportSize = Viewport->GetSize();
	float GameUIScale;

	if (bUseScaledDPI)
	{
		float DPIValue = UserInterfaceSettings->GetDPIScaleBasedOnSize(ScaledDPIViewportReference);
		float ViewportScale = FMath::Min((float)ViewportSize.X / (float)ScaledDPIViewportReference.X, (float)ViewportSize.Y / (float)ScaledDPIViewportReference.Y);

		GameUIScale = DPIValue * ViewportScale;
	}
	else
	{
		GameUIScale = UserInterfaceSettings->GetDPIScaleBasedOnSize(ViewportSize);
	}

	// Remove the platform DPI scale from the incoming size.  Since the platform DPI is already
	// attempt to normalize the UI for a high DPI, and the DPI scale curve is based on raw resolution
	// for what a assumed platform scale of 1, extract that scale the calculated scale, since that will
	// already be applied by slate.
	const float FinalUIScale = GameUIScale / Viewport->GetCachedGeometry().Scale;

	return FinalUIScale;
}

FOptionalSize SGameLayerManager::GetDefaultWindowTitleBarHeight() const
{
	return DefaultWindowTitleBarHeight;
}

void SGameLayerManager::UpdateLayout()
{
	if ( const FSceneViewport* Viewport = SceneViewport.Get() )
	{
		if ( UWorld* World = Viewport->GetClient()->GetWorld() )
		{
			if ( World->IsGameWorld() == false )
			{
				PlayerLayers.Reset();
				return;
			}

			if ( UGameViewportClient* ViewportClient = World->GetGameViewport() )
			{
				const TArray<ULocalPlayer*>& GamePlayers = GEngine->GetGamePlayers(World);

				RemoveMissingPlayerLayers(GamePlayers);
				AddOrUpdatePlayerLayers(CachedGeometry, ViewportClient, GamePlayers);
			}
		}
	}
}

class SPlayerLayer : public SOverlay
{
	SLATE_BEGIN_ARGS(SPlayerLayer)
	{
		_Visibility = EVisibility::SelfHitTestInvisible;
	}
	SLATE_END_ARGS()

public:
	void Construct(const FArguments& InArgs, ULocalPlayer* InOwningPlayer)
	{
		OwningPlayer = InOwningPlayer;

		SOverlay::Construct(SOverlay::FArguments());
	}

	virtual int32 OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const override
	{
		const int32 UserIndex = bEnablePerUserHitTesting && OwningPlayer.IsValid() ? FSlateApplication::Get().GetUserIndexForController(OwningPlayer->GetControllerId()) : INDEX_NONE;

		// Set user index for all widgets beneath this layer to the index of the player that owns this layer
		const int32 OldUserIndex = Args.GetHittestGrid().GetUserIndex();
		Args.GetHittestGrid().SetUserIndex(UserIndex);

		const int32 OutgoingLayer = SOverlay::OnPaint(Args, AllottedGeometry, MyCullingRect, OutDrawElements, LayerId, InWidgetStyle, bParentEnabled);

		// Restore whatever index was set before
		Args.GetHittestGrid().SetUserIndex(OldUserIndex);

		return OutgoingLayer;
	}
private:
	TWeakObjectPtr<ULocalPlayer> OwningPlayer;
};

TSharedPtr<SGameLayerManager::FPlayerLayer> SGameLayerManager::FindOrCreatePlayerLayer(ULocalPlayer* LocalPlayer)
{
	TSharedPtr<FPlayerLayer>* PlayerLayerPtr = PlayerLayers.Find(LocalPlayer);
	if ( PlayerLayerPtr == nullptr )
	{
		// Prevent any navigation outside of a player's layer once focus has been placed there.
		TSharedRef<FNavigationMetaData> StopNavigation = MakeShareable(new FNavigationMetaData());
		StopNavigation->SetNavigationStop(EUINavigation::Up);
		StopNavigation->SetNavigationStop(EUINavigation::Down);
		StopNavigation->SetNavigationStop(EUINavigation::Left);
		StopNavigation->SetNavigationStop(EUINavigation::Right);
		StopNavigation->SetNavigationStop(EUINavigation::Previous);
		StopNavigation->SetNavigationStop(EUINavigation::Next);

		// Create a new entry for the player
		TSharedPtr<FPlayerLayer> NewLayer = MakeShareable(new FPlayerLayer());

		// Create a new overlay widget to house any widgets we want to display for the player.
		NewLayer->Widget = SNew(SPlayerLayer, LocalPlayer)
			.AddMetaData(StopNavigation)
			.Clipping(EWidgetClipping::ClipToBoundsAlways);
		
		// Add the overlay to the player canvas, which we'll update every frame to match
		// the dimensions of the player's split screen rect.
		PlayerCanvas->AddSlot()
			.HAlign(HAlign_Fill)
			.VAlign(VAlign_Fill)
			.Expose(NewLayer->Slot)
			[
				NewLayer->Widget.ToSharedRef()
			];

		PlayerLayerPtr = &PlayerLayers.Add(LocalPlayer, NewLayer);
	}

	return *PlayerLayerPtr;
}

void SGameLayerManager::RemoveMissingPlayerLayers(const TArray<ULocalPlayer*>& GamePlayers)
{
	// Find the player layers for players that no longer exist
	for (auto PlayerLayerIt = PlayerLayers.CreateIterator(); PlayerLayerIt; ++PlayerLayerIt)
	{
		ULocalPlayer* Key = Cast<ULocalPlayer>(PlayerLayerIt.Key().ResolveObjectPtr());
		if ((Key == nullptr) || !GamePlayers.Contains(Key))
		{
			const TSharedPtr<FPlayerLayer> Layer = PlayerLayerIt.Value();
			PlayerCanvas->RemoveSlot(Layer->Widget.ToSharedRef());
			PlayerLayerIt.RemoveCurrent();
		}
	}
}

void SGameLayerManager::RemovePlayerWidgets(ULocalPlayer* LocalPlayer)
{
	TSharedPtr<FPlayerLayer> Layer;
	const FObjectKey LocalPlayerKey(LocalPlayer);
	if (PlayerLayers.RemoveAndCopyValue(LocalPlayerKey, Layer))
	{
		PlayerCanvas->RemoveSlot(Layer->Widget.ToSharedRef());
	}
}

void SGameLayerManager::AddOrUpdatePlayerLayers(const FGeometry& AllottedGeometry, UGameViewportClient* ViewportClient, const TArray<ULocalPlayer*>& GamePlayers)
{
	if (GamePlayers.Num() == 0)
	{
		return;
	}

	ESplitScreenType::Type SplitType = ViewportClient->GetCurrentSplitscreenConfiguration();
	TArray<struct FSplitscreenData>& SplitInfo = ViewportClient->SplitscreenInfo;

	const float InverseDPIScale = ViewportClient->Viewport ? 1.0f / GetGameViewportDPIScale() : 1.0f;

	if (CachedInverseDPIScale != InverseDPIScale)
	{
		Invalidate(EInvalidateWidget::Prepass);
		CachedInverseDPIScale = InverseDPIScale;
	}

	// Add and Update Player Layers
	for ( int32 PlayerIndex = 0; PlayerIndex < GamePlayers.Num(); PlayerIndex++ )
	{
		ULocalPlayer* Player = GamePlayers[PlayerIndex];

		if ( SplitType < SplitInfo.Num() )
		{
			// Viewport Sizes
			FVector2D Position(0.0, 0.0);
			FVector2D Size(0.0, 0.0);

			TSharedPtr<FPlayerLayer> PlayerLayer = FindOrCreatePlayerLayer(Player);
			if (PlayerIndex < SplitInfo[SplitType].PlayerData.Num())
			{
				FPerPlayerSplitscreenData& SplitData = SplitInfo[SplitType].PlayerData[PlayerIndex];
				Size = FVector2D(SplitData.SizeX, SplitData.SizeY);
			}
			else if (SplitType == ESplitScreenType::None && GetDefault<UGameMapsSettings>()->bShowAllPlayerWidgetsWhenSplitscreenDisabled )
			{
				// If splitscreen is disabled, we should overlay all of the player layers in the viewport
				FPerPlayerSplitscreenData& SplitData = SplitInfo[SplitType].PlayerData[0];
				Size = FVector2D(SplitData.SizeX, SplitData.SizeY);
			}		

			GetNormalizeRect(Player, Position, Size);
			Size = Size * AllottedGeometry.GetLocalSize() * InverseDPIScale;
			Position = Position * AllottedGeometry.GetLocalSize() * InverseDPIScale;

			if (WindowTitleBarState.Mode == EWindowTitleBarMode::VerticalBox && Size.Y > WindowTitleBarVerticalBox->GetDesiredSize().Y)
			{
				Size.Y -= WindowTitleBarVerticalBox->GetDesiredSize().Y;
			}

			PlayerLayer->Slot->SetSize(Size);
			PlayerLayer->Slot->SetPosition(Position);
		}
		else
		{
			TSharedPtr<FPlayerLayer> PlayerLayer = PlayerLayers.FindRef(Player);
			if (PlayerLayer.IsValid())
			{
				// Viewport Sizes
				FVector2D Position(0.0, 0.0);
				FVector2D Size(0.0, 0.0);

				PlayerLayer->Slot->SetSize(Size);
				PlayerLayer->Slot->SetPosition(Position);
			}
		}
	}
}

bool SGameLayerManager::GetNormalizeRect(ULocalPlayer* LocalPlayer, FVector2D& OutPosition, FVector2D& OutSize) const
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_SGameLayerManager_GetAspectRatioInset);
	if ( LocalPlayer )
	{
		FSceneViewProjectionData ProjectionData;
		if (LocalPlayer->GetProjectionData(LocalPlayer->ViewportClient->Viewport, ProjectionData))
		{
			const FIntRect ViewRect = ProjectionData.GetViewRect();
			const FIntRect ConstrainedViewRect = ProjectionData.GetConstrainedViewRect();

			const FIntPoint ViewportSize = LocalPlayer->ViewportClient->Viewport->GetSizeXY();

			// Return normalized coordinates.
			OutPosition.X = ConstrainedViewRect.Min.X / (float)ViewportSize.X;
			OutPosition.Y = ConstrainedViewRect.Min.Y / (float)ViewportSize.Y;

			OutSize.X = ConstrainedViewRect.Width() / (float)ViewportSize.X;
			OutSize.Y = ConstrainedViewRect.Height() / (float)ViewportSize.Y;

			return true;
		}
	}

	return false;
}

void SGameLayerManager::SetDefaultWindowTitleBarHeight(float Height)
{
	DefaultWindowTitleBarHeight = Height;
}

void SGameLayerManager::SetWindowTitleBarState(const TSharedPtr<SWidget>& TitleBarContent, EWindowTitleBarMode Mode, bool bTitleBarDragEnabled, bool bWindowButtonsVisible, bool bTitleBarVisible)
{
	UE_LOGF(LogSlate, Log, "Updating window title bar state: %ls mode, drag %ls, window buttons %ls, title bar %ls",
		Mode == EWindowTitleBarMode::Overlay ? TEXT("overlay") : TEXT("vertical box"),
		bTitleBarDragEnabled ? TEXT("enabled") : TEXT("disabled"),
		bWindowButtonsVisible ? TEXT("visible") : TEXT("hidden"),
		bTitleBarVisible ? TEXT("visible") : TEXT("hidden"));
	WindowTitleBarState.ContentWidget = TitleBarContent.IsValid() ? TitleBarContent : DefaultTitleBarContentWidget;
	WindowTitleBarState.Mode = Mode;
	WindowTitleBarState.bTitleBarDragEnabled = bTitleBarDragEnabled;
	WindowTitleBarState.bWindowButtonsVisible = bWindowButtonsVisible;
	WindowTitleBarState.bTitleBarVisible = bTitleBarVisible && bIsGameUsingBorderlessWindow;
	UpdateWindowTitleBar();
}

void SGameLayerManager::RestorePreviousWindowTitleBarState()
{
	// TODO: remove RestorePreviousWindowTitleBarState() and replace its usage in widget blueprints with SetWindowTitleBarState() calls
	SetWindowTitleBarState(nullptr, EWindowTitleBarMode::Overlay, false, false, false);
}

void SGameLayerManager::SetWindowTitleBarVisibility(bool bIsVisible)
{
	WindowTitleBarState.bTitleBarVisible = bIsVisible && bIsGameUsingBorderlessWindow;
	UpdateWindowTitleBarVisibility();
}

TSharedRef<SWidget> SGameLayerManager::AsWidget()
{
	return AsShared();
}

bool SGameLayerManager::CustomPrepass(float LayoutScaleMultiplier)
{
	SetIsProjectContentParent(true);
	return true;
}

void SGameLayerManager::UpdateWindowTitleBar()
{
	if (WindowTitleBarState.ContentWidget.IsValid())
	{
		if (WindowTitleBarState.Mode == EWindowTitleBarMode::Overlay)
		{
			WindowTitleBarOverlay->SetContent(WindowTitleBarState.ContentWidget.ToSharedRef());
			TitleBarAreaOverlay->SetWindowButtonsVisibility(WindowTitleBarState.bWindowButtonsVisible);
		}
		else if (WindowTitleBarState.Mode == EWindowTitleBarMode::VerticalBox)
		{
			WindowTitleBarVerticalBox->SetContent(WindowTitleBarState.ContentWidget.ToSharedRef());
			TitleBarAreaVerticalBox->SetWindowButtonsVisibility(WindowTitleBarState.bWindowButtonsVisible);
		}
	}

	UpdateWindowTitleBarVisibility();
}

void SGameLayerManager::UpdateWindowTitleBarVisibility()
{
	const EVisibility VisibilityWhenEnabled = WindowTitleBarState.bTitleBarDragEnabled ? EVisibility::Visible : EVisibility::SelfHitTestInvisible;
	if (WindowTitleBarState.Mode == EWindowTitleBarMode::Overlay)
	{
		TitleBarAreaOverlay->SetVisibility(WindowTitleBarState.bTitleBarVisible ? VisibilityWhenEnabled : EVisibility::Collapsed);
		TitleBarAreaVerticalBox->SetVisibility(EVisibility::Collapsed);
	}
	else if (WindowTitleBarState.Mode == EWindowTitleBarMode::VerticalBox)
	{
		TitleBarAreaOverlay->SetVisibility(EVisibility::Collapsed);
		TitleBarAreaVerticalBox->SetVisibility(WindowTitleBarState.bTitleBarVisible ? VisibilityWhenEnabled : EVisibility::Collapsed);
	}
}

void SGameLayerManager::RequestToggleFullscreen()
{
	// SWindowTitleBarArea cannot access GEngine, so it'll call this when it needs to toggle fullscreen
	if (GEngine)
	{
		GEngine->DeferredCommands.Add(TEXT("TOGGLE_FULLSCREEN"));
	}
}

void SGameLayerManager::ShowPlayerCanvas(bool bIsVisible)
{
	PlayerCanvas->SetVisibility(bIsVisible ? EVisibility::SelfHitTestInvisible : EVisibility::Collapsed);
}

void SGameLayerManager::ShowDebugCanvas(bool bIsVisible)
{
	DebugCanvas->SetVisibility(bIsVisible ? EVisibility::SelfHitTestInvisible : EVisibility::Collapsed);
}

void SGameLayerManager::ShowViewportSlot(bool bIsVisible)
{
	ViewportSlotContainer->SetVisibility(bIsVisible ? EVisibility::SelfHitTestInvisible : EVisibility::Collapsed);
}

SGameLayerManager::~SGameLayerManager() 
{
#if UE_SLATE_WITH_GAMELAYER_CANVAS_VISIBILITY_COMMANDS
	CanvasVisibility::GameLayerManagerInstances.RemoveSingleSwap(this);
#endif
}
