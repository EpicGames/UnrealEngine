// Copyright Epic Games, Inc. All Rights Reserved.


#include "Slate/SceneViewport.h"
#include "Engine/GameInstance.h"
#include "Engine/GameViewportClient.h"
#include "Engine/GameEngine.h"
#include "Widgets/SViewport.h"
#include "EngineLogs.h"
#include "Misc/App.h"
#include "Input/CursorReply.h"
#include "RenderingThread.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "GameFramework/PlayerController.h"
#include "Engine/Canvas.h"
#include "Engine/RendererSettings.h"
#include "InputKeyEventArgs.h"
#include "Layout/WidgetPath.h"
#include "Misc/CoreDelegates.h"
#include "UnrealEngine.h"
#include "Framework/Application/SlateApplication.h"
#include "Rendering/SlateRenderer.h"
#include "Slate/SlateTextures.h"
#include "Slate/DebugCanvas.h"
#include "IHeadMountedDisplay.h"
#include "IXRTrackingSystem.h"
#include "StereoRenderTargetManager.h"
#include "HDRHelper.h"
#include "StereoRendering.h"
#include "Slate/SlateViewportProvider.h"

DEFINE_LOG_CATEGORY(LogViewport);

extern EWindowMode::Type GetWindowModeType(EWindowMode::Type WindowMode);

FSceneViewport::FSceneViewport(FViewportClient* const InClient, const TSharedPtr<SViewport>& Widget)
	: FSceneViewport(Widget)
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	ViewportClient = InClient;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	if (InClient)
	{
		InClient->AddAssociation(*this);
	}
}

FSceneViewport::FSceneViewport(UScriptViewportClient* const InClient, const TSharedPtr<SViewport>& Widget)
	: FSceneViewport(Widget)
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	ViewportClient = InClient;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	Client = InClient;
	if (InClient)
	{
		InClient->AddAssociation(*this);
	}
}

FSceneViewport::FSceneViewport(const TSharedPtr<SViewport>& InViewportWidget)
	: CurrentReplyState( FReply::Unhandled() )
	, CachedCursorPos(-1, -1)
	, PreCaptureCursorPos(-1, -1)
	, SoftwareCursorPosition( 0, 0 )
	, bIsSoftwareCursorVisible( false )	
	, DebugCanvasDrawer( new FDebugCanvasDrawer )
	, ViewportWidget( InViewportWidget )
	, NumMouseSamplesX( 0 )
	, NumMouseSamplesY( 0 )
	, MouseDelta( 0, 0 )
	, bIsCursorVisible( true )
	, bShouldCaptureMouseOnActivate( true )
	, bRequiresVsync( false )
	, bCameraCut( false )
	, bUseSeparateRenderTarget( InViewportWidget.IsValid() ? !InViewportWidget->ShouldRenderDirectly() : true )
	, bForceSeparateRenderTarget( false )
	, bIsResizing( false )
	, bForceViewportSize(false)
	, bPlayInEditorIsSimulate( false )
	, bCursorHiddenDueToCapture( false )
	, bHDRViewport(false)
	, MousePosBeforeHiddenDueToCapture( -1, -1 )
	, RTTSize( 0, 0 )
	, SceneTargetFormat( EPixelFormat::PF_A2B10G10R10 )
	, NumTouches(0)
	, DisplayColorGamut(EDisplayColorGamut::sRGB_D65)
	, DisplayOutputFormat(EDisplayOutputFormat::SDR_sRGB)
	, MinimumLuminanceInNits(0.f)
	, MaximumLuminanceInNits(100.f)
	, MaximumFullFrameLuminanceInNits(100.f)
	, HDRPaperWhiteInNits(203.f)
	, LimitingColorSpace(UE::Color::EColorSpace::sRGB)
{
	bIsSlateViewport = true;
	RenderThreadSlateTexture = new FSlateRenderTargetRHI(nullptr, 0, 0);
}

FSceneViewport::~FSceneViewport()
{
	Destroy();
	// Wait for resources to be deleted
	FlushRenderingCommands();
}

bool FSceneViewport::HasMouseCapture() const
{
	return ViewportWidget.IsValid() && ViewportWidget.Pin()->HasMouseCapture();
}

bool FSceneViewport::HasFocus() const
{
	return FSlateApplication::Get().GetUserFocusedWidget(0) == ViewportWidget.Pin();
}

void FSceneViewport::CaptureMouse( bool bCapture )
{
	if( bCapture )
	{
		CurrentReplyState.UseHighPrecisionMouseMovement( ViewportWidget.Pin().ToSharedRef() );
	}
	else
	{
		CurrentReplyState.ReleaseMouseCapture();
	}
}

void FSceneViewport::LockMouseToViewport( bool bLock )
{
	if( bLock )
	{
		CurrentReplyState.LockMouseToWidget( ViewportWidget.Pin().ToSharedRef() );
	}
	else
	{
		CurrentReplyState.ReleaseMouseLock();
	}
}

void FSceneViewport::ShowCursor( bool bVisible )
{
	if ( bVisible && !bIsCursorVisible )
	{
		if( bIsSoftwareCursorVisible )
		{
			// Clamp to just inside the viewport for the benefit of unbounded cursor behavior (see UpdateCachedCursorPos())
			const int32 ClampedMouseX = FMath::Clamp<int32>(SoftwareCursorPosition.X, 1, SizeX - 2);
			const int32 ClampedMouseY = FMath::Clamp<int32>(SoftwareCursorPosition.Y, 1, SizeY - 2);

			const FVector2D NormalizedLocalMousePosition = FVector2D(ClampedMouseX, ClampedMouseY) / GetSizeXY();
			const FVector2D AbsolutePosition = CachedGeometry.LocalToAbsolute(NormalizedLocalMousePosition * CachedGeometry.GetLocalSize());
			
			CurrentReplyState.SetMousePos(AbsolutePosition.IntPoint());
			 
			CachedUnboundedCursorPos.Reset();
		}
		else
		{
			// Restore the old mouse position when we show the cursor.
			CurrentReplyState.SetMousePos( PreCaptureCursorPos );
			CachedUnboundedCursorPos.Reset();
		}

		SetPreCaptureMousePosFromSlateCursor();
		bIsCursorVisible = true;
	}
	else if ( !bVisible && bIsCursorVisible )
	{
		// Remember the current mouse position when we hide the cursor.
		SetPreCaptureMousePosFromSlateCursor();
		bIsCursorVisible = false;
	}
}

bool FSceneViewport::SetUserFocus(bool bFocus)
{
	if (bFocus)
	{
		CurrentReplyState.SetUserFocus(ViewportWidget.Pin().ToSharedRef(), EFocusCause::SetDirectly, true);
	}
	else
	{
		CurrentReplyState.ClearUserFocus(true);
	}

	return bFocus;
}

bool FSceneViewport::KeyState( FKey Key ) const
{
	return KeyStateMap.FindRef( Key );
}

void FSceneViewport::Destroy()
{
	check(IsInGameThread());
	// Clear the client while the FSceneViewport-specific state is still alive, so that RemoveAssociation has access to the widget etc.
	SetViewportClient(TStrongPtrVariant<FViewportClient>(nullptr));

	UpdateViewportRHI( true, 0, 0, EWindowMode::Windowed, PF_Unknown );	
}

int32 FSceneViewport::GetMouseX() const
{
	return CachedCursorPos.X;
}

int32 FSceneViewport::GetMouseY() const
{
	return CachedCursorPos.Y;
}

void FSceneViewport::GetMousePos( FIntPoint& MousePosition, const bool bLocalPosition )
{
	if (bLocalPosition)
	{
		MousePosition = CachedCursorPos;
	}
	else
	{
		const FVector2D AbsoluteMousePos = CachedGeometry.LocalToAbsolute(FVector2D(CachedCursorPos.X / CachedGeometry.Scale, CachedCursorPos.Y / CachedGeometry.Scale));
		MousePosition.X = AbsoluteMousePos.X;
		MousePosition.Y = AbsoluteMousePos.Y;
	}
}

void FSceneViewport::GetUnboundedMousePos(FIntPoint& OutUnboundedMousePosition, const bool bLocalPosition)
{
	if (const FIntPoint* UnboundedPosition = CachedUnboundedCursorPos.GetPtrOrNull())
	{
		if (bLocalPosition)
		{
			OutUnboundedMousePosition = *UnboundedPosition;
		}
		else
		{
			const FVector2D AbsoluteMousePos = CachedGeometry.LocalToAbsolute(FVector2D(UnboundedPosition->X / CachedGeometry.Scale, UnboundedPosition->Y / CachedGeometry.Scale));
			OutUnboundedMousePosition.X = AbsoluteMousePos.X;
			OutUnboundedMousePosition.Y = AbsoluteMousePos.Y;
		}
	}
	else
	{
		GetMousePos(OutUnboundedMousePosition, bLocalPosition);
	}
}

void FSceneViewport::SetMouse( int32 X, int32 Y )
{
	const FVector2D NormalizedLocalMousePosition = FVector2D(X, Y) / GetSizeXY();
	FVector2D AbsolutePos = CachedGeometry.LocalToAbsolute(NormalizedLocalMousePosition * CachedGeometry.GetLocalSize());
	FSlateApplication::Get().SetCursorPos( AbsolutePos.RoundToVector() );
	CachedUnboundedCursorPos = CachedCursorPos = FIntPoint(X, Y);
}

void FSceneViewport::ProcessInput( float DeltaTime )
{
	// Required 
}

void FSceneViewport::UpdateCachedCursorPos( const FGeometry& InGeometry, const FPointerEvent& InMouseEvent, bool bIsMoveEvent )
{
	const FPlatformUserId UserId = IPlatformInputDeviceMapper::Get().GetUserForInputDevice(InMouseEvent.GetInputDeviceId());
	if (UserId == FSlateApplication::SlateAppPrimaryPlatformUser)
	{
		FVector2D LocalPixelMousePos = InGeometry.AbsoluteToLocal(InMouseEvent.GetScreenSpacePosition());

		const FVector2D ToViewportPixelScale = UE::Slate::Viewport::GetViewportLocalToPixelScale(InGeometry, FVector2D(SizeX, SizeY));

		LocalPixelMousePos.X = FMath::Clamp(LocalPixelMousePos.X * ToViewportPixelScale.X, (double)TNumericLimits<int32>::Min(), (double)TNumericLimits<int32>::Max());
		LocalPixelMousePos.Y = FMath::Clamp(LocalPixelMousePos.Y * ToViewportPixelScale.Y, (double)TNumericLimits<int32>::Min(), (double)TNumericLimits<int32>::Max());
		
		const FIntPoint NewPos = LocalPixelMousePos.IntPoint();
		const FIntPoint Delta = NewPos - CachedCursorPos;
		
		const int32 PixelWidth = FMath::RoundToInt32(FMath::Clamp(InGeometry.Size.X * ToViewportPixelScale.X, (double)TNumericLimits<int32>::Min(), (double)TNumericLimits<int32>::Max()));
		const int32 PixelHeight = FMath::RoundToInt32(FMath::Clamp(InGeometry.Size.Y * ToViewportPixelScale.Y, (double)TNumericLimits<int32>::Min(), (double)TNumericLimits<int32>::Max()));
		
		CachedCursorPos = NewPos;
		
		if (CachedUnboundedCursorPos.IsSet())
		{
			if (NewPos.X > 0 && NewPos.X < PixelWidth - 1 && NewPos.Y > 0 && NewPos.Y < PixelHeight - 1)
			{
				// When the mouse is over the viewport and not on the edge, the unbound cursor should move in lock step with the normal cursor
				CachedUnboundedCursorPos = CachedUnboundedCursorPos.GetValue() + Delta;
			}
			else if (bIsMoveEvent)
			{
				// When the mouse is outside the viewport or on the edge, rely on straight deltas.
				// This distinction was made due to seeing duplicate OnMouseMove deltas being applied.
				// If that problem can be ameliorated, the above branch should no longer be necessary.
				const FIntPoint UnboundDelta = (InMouseEvent.GetCursorDelta() * ToViewportPixelScale).IntPoint();
				CachedUnboundedCursorPos = CachedUnboundedCursorPos.GetValue() + UnboundDelta;
			}
		}
		else
		{
			// Seed with initial mouse values
			CachedUnboundedCursorPos = CachedCursorPos;
		}
	}
}

void FSceneViewport::UpdateCachedGeometry( const FGeometry& InGeometry )
{
	CachedGeometry = InGeometry;
}

void FSceneViewport::UpdateModifierKeys( const FPointerEvent& InMouseEvent )
{
	KeyStateMap.Add(EKeys::LeftAlt, InMouseEvent.IsLeftAltDown());
	KeyStateMap.Add(EKeys::RightAlt, InMouseEvent.IsRightAltDown());
	KeyStateMap.Add(EKeys::LeftControl, InMouseEvent.IsLeftControlDown());
	KeyStateMap.Add(EKeys::RightControl, InMouseEvent.IsRightControlDown());
	KeyStateMap.Add(EKeys::LeftShift, InMouseEvent.IsLeftShiftDown());
	KeyStateMap.Add(EKeys::RightShift, InMouseEvent.IsRightShiftDown());
	KeyStateMap.Add(EKeys::LeftCommand, InMouseEvent.IsLeftCommandDown());
	KeyStateMap.Add(EKeys::RightCommand, InMouseEvent.IsRightCommandDown());
}

void FSceneViewport::ApplyModifierKeys(const FModifierKeysState& InKeysState, const uint64 Timestamp)
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FViewportClient* ClientPtr = ViewportClient;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	if (ClientPtr && GetSizeXY() != FIntPoint::ZeroValue)
	{
		const FInputDeviceId DefaultInputDevice = IPlatformInputDeviceMapper::Get().GetDefaultInputDevice();

		// Switch to the viewport clients world before processing input
		FScopedConditionalWorldSwitcher WorldSwitcher(ClientPtr);

		if (InKeysState.IsLeftAltDown())
		{
			ClientPtr->InputKey(FInputKeyEventArgs(this, DefaultInputDevice, EKeys::LeftAlt, IE_Pressed, Timestamp));
		}
		if (InKeysState.IsRightAltDown())
		{
			ClientPtr->InputKey(FInputKeyEventArgs(this, DefaultInputDevice, EKeys::RightAlt, IE_Pressed, Timestamp));
		}
		if (InKeysState.IsLeftControlDown())
		{
			ClientPtr->InputKey(FInputKeyEventArgs(this, DefaultInputDevice, EKeys::LeftControl, IE_Pressed, Timestamp));
		}
		if (InKeysState.IsRightControlDown())
		{
			ClientPtr->InputKey(FInputKeyEventArgs(this, DefaultInputDevice, EKeys::RightControl, IE_Pressed, Timestamp));
		}
		if (InKeysState.IsLeftShiftDown())
		{
			ClientPtr->InputKey(FInputKeyEventArgs(this, DefaultInputDevice, EKeys::LeftShift, IE_Pressed, Timestamp));
		}
		if (InKeysState.IsRightShiftDown())
		{
			ClientPtr->InputKey(FInputKeyEventArgs(this, DefaultInputDevice, EKeys::RightShift, IE_Pressed, Timestamp));
		}
	}
}

void FSceneViewport::ProcessAccumulatedPointerInput()
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FViewportClient* ClientPtr = ViewportClient;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	if (!ClientPtr)
	{
		return;
	}

	// Switch to the viewport clients world before processing input
	FScopedConditionalWorldSwitcher WorldSwitcher(ClientPtr);

	const bool bViewportHasCapture = ViewportWidget.IsValid() && ViewportWidget.Pin()->HasMouseCapture();

	ClientPtr->ProcessAccumulatedPointerInput(this);

	if (NumMouseSamplesX > 0 || NumMouseSamplesY > 0)
	{
		const float DeltaTime = FApp::GetDeltaTime();
		FInputDeviceId DefaultInputDevice = IPlatformInputDeviceMapper::Get().GetDefaultInputDevice();
		// TODO: Populate this with a mouse timestamp
		const uint64 MoseRecentMouseTimestamp = 0u;
		
		FInputKeyEventArgs MouseXArgs (this, DefaultInputDevice, EKeys::MouseX, MouseDelta.X, DeltaTime, NumMouseSamplesX, MoseRecentMouseTimestamp);
		ClientPtr->InputAxis(MouseXArgs);

		FInputKeyEventArgs MouseYArgs (this, DefaultInputDevice, EKeys::MouseY, MouseDelta.Y, DeltaTime, NumMouseSamplesY, MoseRecentMouseTimestamp);
		ClientPtr->InputAxis(MouseYArgs);
	}

	if ( bCursorHiddenDueToCapture )
	{
		switch (ClientPtr->GetMouseCaptureMode())
		{
		case EMouseCaptureMode::NoCapture:
		case EMouseCaptureMode::CaptureDuringMouseDown:
		case EMouseCaptureMode::CaptureDuringRightMouseDown:
			if ( !bViewportHasCapture )
			{
				bool bShouldMouseBeVisible = ClientPtr->GetCursor(this, GetMouseX(), GetMouseY()) != EMouseCursor::None;

				UWorld* World = ClientPtr->GetWorld();
				if ( World && World->IsGameWorld() && World->GetGameInstance() )
				{
					APlayerController* PC = World->GetGameInstance()->GetFirstLocalPlayerController();
					bShouldMouseBeVisible &= PC && PC->ShouldShowMouseCursor();
				}

				if ( bShouldMouseBeVisible )
				{
					bCursorHiddenDueToCapture = false;
					CurrentReplyState.SetMousePos(MousePosBeforeHiddenDueToCapture);
					MousePosBeforeHiddenDueToCapture = FIntPoint(-1, -1);
				}
			}
			break;
		}
	}

	MouseDelta = FIntPoint::ZeroValue;
	NumMouseSamplesX = 0;
	NumMouseSamplesY = 0;
}

FVector2D FSceneViewport::VirtualDesktopPixelToViewport(FIntPoint VirtualDesktopPointPx) const
{
	// Virtual Desktop Pixel to local slate unit
	const FVector2D TransformedPoint = CachedGeometry.AbsoluteToLocal(FVector2D(VirtualDesktopPointPx.X, VirtualDesktopPointPx.Y));

	// Pixels to normalized coordinates and correct for DPI scale
	return FVector2D( TransformedPoint.X / SizeX * CachedGeometry.Scale, TransformedPoint.Y / SizeY * CachedGeometry.Scale);
}

FIntPoint FSceneViewport::ViewportToVirtualDesktopPixel(FVector2D ViewportCoordinate) const
{
	// Normalized to pixels transform
	const FVector2D LocalCoordinateInSu = FVector2D( ViewportCoordinate.X * SizeX, ViewportCoordinate.Y * SizeY );
	// Local slate unit to virtual desktop pixel.
	const FVector2D TransformedPoint = FVector2D( CachedGeometry.LocalToAbsolute( LocalCoordinateInSu ) );

	// Correct for DPI
	return FIntPoint( FMath::TruncToInt(TransformedPoint.X / CachedGeometry.Scale), FMath::TruncToInt(TransformedPoint.Y / CachedGeometry.Scale) );
}

IStereoRenderTargetManager* RetrieveStereoRenderTargetManager(bool bIsStereoRenderingAllowed)
{
	return (bIsStereoRenderingAllowed && GEngine->StereoRenderingDevice.IsValid() && GEngine->StereoRenderingDevice->IsStereoEnabledOnNextFrame())
		   ? GEngine->StereoRenderingDevice->GetRenderTargetManager()
		   : nullptr;

}

static void ComputeSceneViewportHDRMetaData(FHDRMetaData& OutHDRMetaData, const FVector2D& WindowTopLeft, const FVector2D& WindowBottomRight, void* OSWindow, bool bIsStereoRenderingAllowed)
{
	// @todo vreditor switch: This code needs to be called when switching between stereo/non when going immersive.  Seems to always work out that way anyway though? (Probably due to resize)
	IStereoRenderTargetManager* const StereoRenderTargetManager = RetrieveStereoRenderTargetManager(bIsStereoRenderingAllowed);
	if (StereoRenderTargetManager && StereoRenderTargetManager->HDRGetMetaDataForStereo(OutHDRMetaData.DisplayOutputFormat, OutHDRMetaData.DisplayColorGamut, OutHDRMetaData.bHDRSupported))
	{
		// TODO: Get the real values for stereo.
		OutHDRMetaData.MinimumLuminanceInNits = OutHDRMetaData.bHDRSupported ? .1f : 0.f;
		OutHDRMetaData.MaximumLuminanceInNits = OutHDRMetaData.bHDRSupported ? 1000.f : 100.f;
		OutHDRMetaData.MaximumFullFrameLuminanceInNits = OutHDRMetaData.bHDRSupported ? 600.f : 100.f;
		OutHDRMetaData.HDRPaperWhiteInNits = 203.f;
		OutHDRMetaData.LimitingColorSpace = OutHDRMetaData.bHDRSupported ? UE::Color::FColorSpace::GetP3D65() : UE::Color::FColorSpace::GetSRGB();
		return;
	}

	HDRGetMetaDataForWindow(
		OutHDRMetaData,
		WindowTopLeft,
		WindowBottomRight,
		OSWindow);
}

namespace UE::Private
{
	namespace
	{
		using namespace UE::Slate;

		class FSceneViewportPaintScope final : public FPaintScope
		{
		public:
			explicit FSceneViewportPaintScope(const FViewportClient* const Client)
				// FSceneViewport defines a relationship between an FSceneInterface and a portion of the Slate hierarchy (children of the
				// corresponding SViewport), therefore it is responsible for pushing its scene onto the stack so that any child widgets that use
				// Material Parameter Collection will resolve to the MPC instance associated with that scene.
				: SceneScope(MakeSceneRegistration(Client))
			{
			}

		private:
			static FSceneRegistrationScope MakeSceneRegistration(const FViewportClient* const Client)
			{
				const UWorld* const World = Client ? Client->GetWorld() : nullptr;
				return FSlateApplication::Get().GetRenderer()->RegisterCurrentScene(World ? World->Scene : nullptr);
			}

			FSceneRegistrationScope SceneScope;
		};
	}
}

TUniquePtr<UE::Slate::FPaintScope> FSceneViewport::Paint(const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, const int32 LayerId, const FWidgetStyle& InWidgetStyle, const bool bParentEnabled)
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FViewportClient* const ClientPtr = ViewportClient;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	// Switch to the viewport clients world before resizing
	FScopedConditionalWorldSwitcher WorldSwitcher(ClientPtr);

	TUniquePtr PaintScope = MakeUnique<UE::Private::FSceneViewportPaintScope>(ClientPtr);

	/** Check to see if the viewport should be resized */
	if (!bForceViewportSize)
	{
		// When a viewport element is created, its coordinates are rounded after applying the render transform.
		// Considering that both absolute position and the size of the viewport can be non-integers,
		// the correct draw size of the viewport can only be determined by rounding its absolute coordinates.
		FVector2D TopLeft = AllottedGeometry.GetAbsolutePosition();
		FVector2D BottomRight = TopLeft + AllottedGeometry.GetDrawSize();
		FIntPoint DrawSize = FIntPoint(FMath::RoundToInt(BottomRight.X) - FMath::RoundToInt(TopLeft.X), FMath::RoundToInt(BottomRight.Y) - FMath::RoundToInt(TopLeft.Y));
		bool bIsHDREnabled = IsHDREnabled();

		SWindow* PaintWindow = OutDrawElements.GetPaintWindow();
		bool bHDRStale = (bIsHDREnabled != bHDRViewport);
		if (PaintWindow)
		{
			FHDRMetaData OutHDRMetaData;
			ComputeSceneViewportHDRMetaData(
				OutHDRMetaData,
				PaintWindow->GetPositionInScreen(),
				PaintWindow->GetPositionInScreen() + PaintWindow->GetSizeInScreen(),
				PaintWindow->GetNativeWindow()->GetOSWindowHandle(),
				IsStereoRenderingAllowed());
			// if we manage to get data for the window, we can ignore the global toggle IsHDREnabled since HDRGetMetaData will take both the global flag and the monitor properties
			bHDRStale = DisplayOutputFormat != OutHDRMetaData.DisplayOutputFormat;
			bHDRStale |= DisplayColorGamut != OutHDRMetaData.DisplayColorGamut;
			bHDRStale |= MinimumLuminanceInNits != OutHDRMetaData.MinimumLuminanceInNits;
			bHDRStale |= MaximumLuminanceInNits != OutHDRMetaData.MaximumLuminanceInNits;
			bHDRStale |= MaximumFullFrameLuminanceInNits != OutHDRMetaData.MaximumFullFrameLuminanceInNits;
			bHDRStale |= HDRPaperWhiteInNits != OutHDRMetaData.HDRPaperWhiteInNits;
			bHDRStale |= LimitingColorSpace != OutHDRMetaData.LimitingColorSpace;
			bHDRStale |= bHDRViewport != OutHDRMetaData.bHDRSupported;
		}

	    if (GetSizeXY() != DrawSize || bHDRStale)
		{
			if (TSharedPtr<SWindow> Window = FindWindow())
			{
				//@HACK VREDITOR
				//check(Window.IsValid());
                // This makes sure that by the time when we call HDRGetMetaData again in UpdateViewportRHI, we still provide the same inputs so we don't re-create the swapchain on a per-frame basis
				ensure(PaintWindow == Window.Get());

				// In the stereo case, the HMD display size drives the base RT size, separate from the PIE mirror window
				const bool bResizeTargetValid = Window->IsViewportSizeDrivenByWindow() || 
					(GIsEditor && IsStereoRenderingAllowed());

				if (bResizeTargetValid)
				{
					if (ViewportWidget.Pin()->ShouldRenderDirectly())
					{
						InitialPositionX = FMath::Max(0.0f, AllottedGeometry.AbsolutePosition.X);
						InitialPositionY = FMath::Max(0.0f, AllottedGeometry.AbsolutePosition.Y);
					}

					ResizeViewport(FMath::Max(0, DrawSize.X), FMath::Max(0, DrawSize.Y), Window->GetWindowMode());
				}
			}
		}

		// Because DrawSize and GetSizeXY() are fixed for VR PIE after launch, we need to manually check if a viewport resize is necessary due to window resizes.
		// This ensures the spectator view will be stretched to fit to the newly resized window without changing the size of the HMD render target.
		else if (GIsEditor && IsStereoRenderingAllowed())
		{
			if (TSharedPtr<SWindow> Window = FindWindow())
			{
				ensure(PaintWindow == Window.Get());

				FSlateApplication::Get().GetRenderer()->UpdateFullscreenState(Window.ToSharedRef());
			}
		}
	}

	return PaintScope;
}

bool FSceneViewport::IsForegroundWindow() const
{
	bool bIsForeground = false;
	if( ViewportWidget.IsValid() )
	{
		if (TSharedPtr<SWindow> Window = FindWindow())
		{
			bIsForeground = Window->GetNativeWindow()->IsForegroundWindow();
		}
	}

	return bIsForeground;
}

FCursorReply FSceneViewport::OnCursorQuery( const FGeometry& MyGeometry, const FPointerEvent& CursorEvent )
{
	if (bCursorHiddenDueToCapture)
	{
		return FCursorReply::Cursor(EMouseCursor::None);
	}

	EMouseCursor::Type MouseCursorToUse = EMouseCursor::Default;

	// If the cursor should be hidden, use EMouseCursor::None,
	// only when in the foreground, or we'll hide the mouse in the window/program above us.
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FViewportClient* ClientPtr = ViewportClient;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	if (ClientPtr && GetSizeXY() != FIntPoint::ZeroValue)
	{
		const int32 MouseX = GetMouseX();
		const int32 MouseY = GetMouseY();
		MouseCursorToUse = ClientPtr->GetCursor(this, MouseX, MouseY);
	}

	// In game mode we may be using a borderless window, which needs OnCursorQuery call to handle window resize cursors
	if (IsRunningGame() && GEngine && GEngine->GameViewport && MouseCursorToUse != EMouseCursor::None)
	{
		TSharedPtr<SWindow> Window = GEngine->GameViewport->GetWindow();
		if (Window.IsValid())
		{
			FCursorReply Reply = Window->OnCursorQuery(MyGeometry, CursorEvent);
			if (Reply.IsEventHandled())
			{
				return Reply;
			}
		}
	}

	// Use the default cursor if there is no viewport client or we dont have focus
	return FCursorReply::Cursor(MouseCursorToUse);
}

TOptional<TSharedRef<SWidget>> FSceneViewport::OnMapCursor(const FCursorReply& CursorReply)
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FViewportClient* ClientPtr = ViewportClient;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	if (ClientPtr && GetSizeXY() != FIntPoint::ZeroValue)
	{
		return ClientPtr->MapCursor(this, CursorReply);
	}
	return ISlateViewport::OnMapCursor(CursorReply);
}

FReply FSceneViewport::OnMouseButtonDown( const FGeometry& InGeometry, const FPointerEvent& InMouseEvent )
{
	// Start a new reply state
	// Prevent throttling when interacting with the viewport so we can move around in it
	CurrentReplyState = FReply::Handled().PreventThrottling();

	KeyStateMap.Add(InMouseEvent.GetEffectingButton(), true);
	UpdateModifierKeys(InMouseEvent);

	UpdateCachedGeometry(InGeometry);
	UpdateCachedCursorPos(InGeometry, InMouseEvent);
	
	const uint32 DownButtonsCount = InMouseEvent.IsMouseButtonDown(EKeys::LeftMouseButton)
								  + InMouseEvent.IsMouseButtonDown(EKeys::MiddleMouseButton)
								  + InMouseEvent.IsMouseButtonDown(EKeys::RightMouseButton);

	// This is the first pressed button: initialize unbounded cursor position to match current cursor position
	if (DownButtonsCount == 1)
	{
		CachedUnboundedCursorPos = CachedCursorPos;
	}

	// Switch to the viewport clients world before processing input
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FViewportClient* ClientPtr = ViewportClient;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	FScopedConditionalWorldSwitcher WorldSwitcher(ClientPtr);
	if (ClientPtr && GetSizeXY() != FIntPoint::ZeroValue)
	{
		// If we're obtaining focus, we have to copy the modifier key states prior to processing this mouse button event, as this is the only point at which the mouse down
		// event is processed when focus initially changes and the modifier keys need to be in-place to detect any unique drag-like events.
		if (!HasFocus())
		{
			FModifierKeysState KeysState = FSlateApplication::Get().GetModifierKeys();
			ApplyModifierKeys(KeysState, InMouseEvent.GetEventTimestamp());
		}

		const bool bTemporaryCapture =
			ClientPtr->GetMouseCaptureMode() == EMouseCaptureMode::CaptureDuringMouseDown ||
			(ClientPtr->GetMouseCaptureMode() == EMouseCaptureMode::CaptureDuringRightMouseDown && InMouseEvent.GetEffectingButton() == EKeys::RightMouseButton);

		// Process primary input if we aren't currently a game viewport, we already have capture, or we are permanent capture that doesn't consume the mouse down.
		const bool bProcessInputPrimary = !IsCurrentlyGameViewport() || HasMouseCapture()
			|| ClientPtr->GetMouseCaptureMode() == EMouseCaptureMode::CapturePermanently_IncludingInitialMouseDown;

		const bool bAnyMenuWasVisible = FSlateApplication::Get().AnyMenusVisible();

		// Process the mouse event
		if (bTemporaryCapture || bProcessInputPrimary)
		{
			if (!ClientPtr->InputKey(FInputKeyEventArgs(this, InMouseEvent.GetInputDeviceId(), InMouseEvent.GetEffectingButton(), IE_Pressed, 1.0f, InMouseEvent.IsTouchEvent(), InMouseEvent.GetEventTimestamp())))
			{
				CurrentReplyState = FReply::Unhandled();
			}
		}

		// a new menu was opened if there was previously not a menu visible but now there is
		const bool bNewMenuWasOpened = !bAnyMenuWasVisible && FSlateApplication::Get().AnyMenusVisible();

		const bool bPermanentCapture =
			ClientPtr->GetMouseCaptureMode() == EMouseCaptureMode::CapturePermanently ||
			ClientPtr->GetMouseCaptureMode() == EMouseCaptureMode::CapturePermanently_IncludingInitialMouseDown;

		if (FSlateApplication::Get().IsActive() && !ClientPtr->IgnoreInput() &&
			!bNewMenuWasOpened && // We should not focus the viewport if a menu was opened as it would close the menu
			(bPermanentCapture || bTemporaryCapture))
		{
			CurrentReplyState = AcquireFocusAndCapture(FIntPoint(InMouseEvent.GetScreenSpacePosition().X, InMouseEvent.GetScreenSpacePosition().Y), EFocusCause::Mouse);
		}
	}

	// Re-set prevent throttling here as it can get reset when inside of InputKey()
	CurrentReplyState.PreventThrottling();
	return CurrentReplyState;
}

FReply FSceneViewport::AcquireFocusAndCapture(FIntPoint MousePosition, EFocusCause FocusCause)
{
	bShouldCaptureMouseOnActivate = false;

	FReply ReplyState = FReply::Handled().PreventThrottling();

	TSharedRef<SViewport> ViewportWidgetRef = ViewportWidget.Pin().ToSharedRef();

	// Mouse down should focus viewport for user input
	ReplyState.SetUserFocus(ViewportWidgetRef, FocusCause);

	FViewportClient& ViewportClientRef = GetClientChecked();
	UWorld* World = ViewportClientRef.GetWorld();
	if (World && World->IsGameWorld() && World->GetGameInstance() && (World->GetGameInstance()->GetFirstLocalPlayerController() || World->IsPlayInEditor()))
	{
		ReplyState.CaptureMouse(ViewportWidgetRef);

		if (ViewportClientRef.LockDuringCapture())
		{
			ReplyState.LockMouseToWidget(ViewportWidgetRef);
		}

		APlayerController* PC = World->GetGameInstance()->GetFirstLocalPlayerController();
		const bool bShouldShowMouseCursor = PC && PC->ShouldShowMouseCursor();

		if (ViewportClientRef.HideCursorDuringCapture() && bShouldShowMouseCursor)
		{
			bCursorHiddenDueToCapture = true;
			MousePosBeforeHiddenDueToCapture = MousePosition;
			
			// The slate app will correct mouse positions for non-standard screen / viewport resolution combos
			// We want to save the mouse position pre-correction so it isn't applied twice when restoring mouse position
			TSharedPtr<SWindow> Window = FindWindow();
			if (FSlateApplication::Get().GetTransformFullscreenMouseInput() && !GIsEditor && Window.IsValid() && Window->GetWindowMode() == EWindowMode::Fullscreen)
			{
				const FDisplayMetrics& CachedDisplayMetrics = FSlateApplication::Get().GetCachedDisplayMetricsByRef();
				const FVector2f DisplaySize = CachedDisplayMetrics.MonitorInfo.IsEmpty()
					? FVector2f((float)CachedDisplayMetrics.PrimaryDisplayWidth, (float)CachedDisplayMetrics.PrimaryDisplayHeight)
					: Window->GetFullScreenInfo().GetSize();
				const FVector2f CorrectionScale = DisplaySize / Window->GetSizeInScreen();
				const FVector2f PositionInScreen = Window->GetPositionInScreen();
				FVector2f AdjustedMousePosition = FVector2f(MousePosition);
				AdjustedMousePosition -= PositionInScreen;
				AdjustedMousePosition *= CorrectionScale;
				AdjustedMousePosition += PositionInScreen;
				MousePosBeforeHiddenDueToCapture = AdjustedMousePosition.IntPoint();
			}
		}

		if ( bCursorHiddenDueToCapture || !bShouldShowMouseCursor )
		{
			ReplyState.UseHighPrecisionMouseMovement(ViewportWidgetRef);
		}
	}
	else
	{
		ReplyState.UseHighPrecisionMouseMovement(ViewportWidgetRef);
	}

	return ReplyState;
}

bool FSceneViewport::IsCurrentlyGameViewport()
{
	// Either were game code only or were are currently play in editor.
	return (FApp::IsGame() && !GIsEditor && GetClient() == GEngine->GameViewport) || IsPlayInEditorViewport();
}

FReply FSceneViewport::OnMouseButtonUp( const FGeometry& InGeometry, const FPointerEvent& InMouseEvent )
{
	// Start a new reply state
	CurrentReplyState = FReply::Handled();

	KeyStateMap.Add( InMouseEvent.GetEffectingButton(), false );
	UpdateModifierKeys( InMouseEvent );

	UpdateCachedGeometry(InGeometry);
	UpdateCachedCursorPos( InGeometry, InMouseEvent );
	

	// Switch to the viewport clients world before processing input
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FViewportClient* ClientPtr = ViewportClient;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	FScopedConditionalWorldSwitcher WorldSwitcher(ClientPtr);
	bool bCursorVisible = true;
	bool bReleaseMouseCapture = true;

	if (ClientPtr && GetSizeXY() != FIntPoint::ZeroValue )
	{
		if (!ClientPtr->InputKey(FInputKeyEventArgs(this, InMouseEvent.GetInputDeviceId(), InMouseEvent.GetEffectingButton(), IE_Released, 1.0f, InMouseEvent.IsTouchEvent(), InMouseEvent.GetEventTimestamp())))
		{
			CurrentReplyState = FReply::Unhandled();
		}

		bCursorVisible = ClientPtr->GetCursor(this, GetMouseX(), GetMouseY()) != EMouseCursor::None;

		if (bCursorVisible)
		{
			bReleaseMouseCapture = true;
			UE_LOGF(LogViewport, Verbose, "Releasing Mouse Capture; Cursor is visible");
		}
		else if (ClientPtr->GetMouseCaptureMode() == EMouseCaptureMode::CaptureDuringMouseDown)
		{
			bReleaseMouseCapture = true;
			UE_LOGF(LogViewport, Verbose, "Releasing Mouse Capture; EMouseCaptureMode::CaptureDuringMouseDown - Mouse Button Released");
		}
		else if (ClientPtr->GetMouseCaptureMode() == EMouseCaptureMode::CaptureDuringRightMouseDown && InMouseEvent.GetEffectingButton() == EKeys::RightMouseButton)
		{
			bReleaseMouseCapture = true;
			UE_LOGF(LogViewport, Verbose, "Releasing Mouse Capture; EMouseCaptureMode::CaptureDuringRightMouseDown - Right Mouse Button Released");
		}
		else
		{
			bReleaseMouseCapture = false;
		}
	}

	if ( !IsCurrentlyGameViewport() || bReleaseMouseCapture )
	{
		// On mouse up outside of the game (editor viewport) or if the cursor is visible in game, we should make sure the mouse is no longer captured
		// as long as the left/middle/right mouse buttons are not still down
		if ( !InMouseEvent.IsMouseButtonDown(EKeys::RightMouseButton) && !InMouseEvent.IsMouseButtonDown(EKeys::LeftMouseButton) && !InMouseEvent.IsMouseButtonDown(EKeys::MiddleMouseButton) )
		{
			if ( bCursorHiddenDueToCapture )
			{
				bCursorHiddenDueToCapture = false;
				CurrentReplyState.SetMousePos(MousePosBeforeHiddenDueToCapture);
				MousePosBeforeHiddenDueToCapture = FIntPoint(-1, -1);
			}

			CurrentReplyState.ReleaseMouseCapture();

			if (bCursorVisible && !ClientPtr->ShouldAlwaysLockMouse())
			{
				CurrentReplyState.ReleaseMouseLock();
			}
		}
	}

	return CurrentReplyState;
}

void FSceneViewport::OnMouseEnter( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent )
{
	UpdateCachedCursorPos( MyGeometry, MouseEvent );
	GetClientChecked().MouseEnter(this, GetMouseX(), GetMouseY());
}

void FSceneViewport::OnMouseLeave( const FPointerEvent& MouseEvent )
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	if (FViewportClient* ClientPtr = ViewportClient)
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	{
		ClientPtr->MouseLeave(this);

		if (IsCurrentlyGameViewport())
		{
			CachedCursorPos = FIntPoint(-1, -1);
		}
	}
}

FReply FSceneViewport::OnMouseMove( const FGeometry& InGeometry, const FPointerEvent& InMouseEvent )
{
	// Start a new reply state
	CurrentReplyState = FReply::Handled();

	UpdateCachedGeometry(InGeometry);
	
	constexpr bool bIsMoveEvent = true;
	UpdateCachedCursorPos(InGeometry, InMouseEvent, bIsMoveEvent);

	const bool bViewportHasCapture = ViewportWidget.IsValid() && ViewportWidget.Pin()->HasMouseCapture();
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FViewportClient* ClientPtr = ViewportClient;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	if (ClientPtr && GetSizeXY() != FIntPoint::ZeroValue)
	{
		// Switch to the viewport clients world before processing input
		FScopedConditionalWorldSwitcher WorldSwitcher(ClientPtr);

		if ( bViewportHasCapture )
		{
			ClientPtr->CapturedMouseMove(this, GetMouseX(), GetMouseY());
		}
		else
		{
			ClientPtr->MouseMove(this, GetMouseX(), GetMouseY());
		}

		if ( bViewportHasCapture )
		{
			// Accumulate delta changes to mouse movement.  Depending on the sample frequency of a mouse we may get many per frame.
			//@todo Slate: In directinput, number of samples in x/y could be different...
			const FVector2D CursorDelta = InMouseEvent.GetCursorDelta();
			MouseDelta.X += CursorDelta.X;
			++NumMouseSamplesX;

			MouseDelta.Y -= CursorDelta.Y;
			++NumMouseSamplesY;
		}

		if ( bCursorHiddenDueToCapture )
		{
			// If hidden during capture, don't actually move the cursor
			FVector2D RevertedCursorPos( MousePosBeforeHiddenDueToCapture.X, MousePosBeforeHiddenDueToCapture.Y );
			FSlateApplication::Get().SetCursorPos(RevertedCursorPos);
		}
	}

	return CurrentReplyState;
}

FReply FSceneViewport::OnMouseWheel( const FGeometry& InGeometry, const FPointerEvent& InMouseEvent )
{
	// Start a new reply state
	CurrentReplyState = FReply::Handled();

	UpdateCachedGeometry(InGeometry);
	UpdateCachedCursorPos( InGeometry, InMouseEvent );

PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FViewportClient* ClientPtr = ViewportClient;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	if (ClientPtr  && GetSizeXY() != FIntPoint::ZeroValue )
	{
		// Switch to the viewport clients world before processing input
		FScopedConditionalWorldSwitcher WorldSwitcher(ClientPtr);

		// The viewport client accepts two different keys depending on the direction of scroll.
		FKey const ViewportClientKey = InMouseEvent.GetWheelDelta() < 0 ? EKeys::MouseScrollDown : EKeys::MouseScrollUp;

		// Pressed and released should be sent
		ClientPtr->InputKey(FInputKeyEventArgs(this, InMouseEvent.GetInputDeviceId(), ViewportClientKey, IE_Pressed, 1.0f, InMouseEvent.IsTouchEvent(), InMouseEvent.GetEventTimestamp()));
		ClientPtr->InputKey(FInputKeyEventArgs(this, InMouseEvent.GetInputDeviceId(), ViewportClientKey, IE_Released, 1.0f, InMouseEvent.IsTouchEvent(), InMouseEvent.GetEventTimestamp()));
		ClientPtr->InputAxis(FInputKeyEventArgs(this, InMouseEvent.GetInputDeviceId(), EKeys::MouseWheelAxis, InMouseEvent.GetWheelDelta(), FApp::GetDeltaTime(), 1, InMouseEvent.GetEventTimestamp()));
	}
	return CurrentReplyState;
}

FReply FSceneViewport::OnMouseButtonDoubleClick( const FGeometry& InGeometry, const FPointerEvent& InMouseEvent )
{
	// Start a new reply state
	CurrentReplyState = FReply::Handled(); 

	// Note: When double-clicking, the following message sequence is sent:
	//	WM_*BUTTONDOWN
	//	WM_*BUTTONUP
	//	WM_*BUTTONDBLCLK	(Needs to set the KeyStates[*] to true)
	//	WM_*BUTTONUP
	KeyStateMap.Add( InMouseEvent.GetEffectingButton(), true );

	UpdateCachedGeometry(InGeometry);
	UpdateCachedCursorPos(InGeometry, InMouseEvent);

PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FViewportClient* ClientPtr = ViewportClient;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	if (ClientPtr && GetSizeXY() != FIntPoint::ZeroValue)
	{
		// Switch to the viewport clients world before processing input
		FScopedConditionalWorldSwitcher WorldSwitcher(ClientPtr);

		if (!ClientPtr->InputKey(FInputKeyEventArgs(this, InMouseEvent.GetInputDeviceId(), InMouseEvent.GetEffectingButton(), IE_DoubleClick, 1.0f, InMouseEvent.IsTouchEvent(), InMouseEvent.GetEventTimestamp())))
		{
			CurrentReplyState = FReply::Unhandled();
		}
	}
	return CurrentReplyState;
}

FReply FSceneViewport::OnTouchStarted( const FGeometry& MyGeometry, const FPointerEvent& TouchEvent )
{
	// Start a new reply state
	CurrentReplyState = FReply::Handled().PreventThrottling(); 
	++NumTouches;

	UpdateCachedGeometry(MyGeometry);
	UpdateCachedCursorPos(MyGeometry, TouchEvent);

PRAGMA_DISABLE_DEPRECATION_WARNINGS
	if (FViewportClient* ClientPtr = ViewportClient)
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	{
		// Switch to the viewport clients world before processing input
		FScopedConditionalWorldSwitcher WorldSwitcher(ClientPtr);

		const FVector2D TouchPosition = CachedCursorPos;

		const bool bEventConsumed = ClientPtr->InputTouch(this,
			FTouchId(TouchEvent.GetInputDeviceId(), static_cast<ETouchIndex::Type>(TouchEvent.GetPointerIndex())), ETouchType::Began, TouchPosition,
			TouchEvent.GetTouchForce(), TouchEvent.GetEventTimestamp());
		if (bEventConsumed)
		{
			if (ClientPtr->GetMouseCaptureMode() == EMouseCaptureMode::CaptureDuringMouseDown)
			{
				CurrentReplyState = AcquireFocusAndCapture(FIntPoint(TouchEvent.GetScreenSpacePosition().X, TouchEvent.GetScreenSpacePosition().Y), EFocusCause::Mouse);
			}
		}
		else
		{
			CurrentReplyState = FReply::Unhandled(); 
		}
	}

	return CurrentReplyState;
}

FReply FSceneViewport::OnTouchMoved( const FGeometry& MyGeometry, const FPointerEvent& TouchEvent )
{
	// Start a new reply state
	CurrentReplyState = FReply::Handled();

	UpdateCachedGeometry(MyGeometry);
	
	constexpr bool bIsMoveEvent = true;
	UpdateCachedCursorPos(MyGeometry, TouchEvent, bIsMoveEvent);
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	if (FViewportClient* ClientPtr = ViewportClient)
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	{
		// Switch to the viewport clients world before processing input
		FScopedConditionalWorldSwitcher WorldSwitcher(ClientPtr);

		const bool bEventConsumed = ClientPtr->InputTouch(this,
			FTouchId(TouchEvent.GetInputDeviceId(), static_cast<ETouchIndex::Type>(TouchEvent.GetPointerIndex())), ETouchType::Moved, CachedCursorPos,
			TouchEvent.GetTouchForce(), TouchEvent.GetEventTimestamp());
		if (!bEventConsumed)
		{
			CurrentReplyState = FReply::Unhandled(); 
		}
	}

	return CurrentReplyState;
}

FReply FSceneViewport::OnTouchEnded( const FGeometry& MyGeometry, const FPointerEvent& TouchEvent )
{
	// Start a new reply state
	CurrentReplyState = FReply::Handled();

	FIntPoint CurCursorPos;

	UpdateCachedGeometry(MyGeometry);
	if (--NumTouches > 0)
	{
		UpdateCachedCursorPos(MyGeometry, TouchEvent);
		CurCursorPos = CachedCursorPos;
	}
	else
	{
		UpdateCachedCursorPos(MyGeometry, TouchEvent);
		CurCursorPos = CachedCursorPos;
		CachedCursorPos = FIntPoint(-1, -1);
	}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
	if (FViewportClient* ClientPtr = ViewportClient)
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	{
		// Switch to the viewport clients world before processing input
		FScopedConditionalWorldSwitcher WorldSwitcher(ClientPtr);

		const bool bEventConsumed = ClientPtr->InputTouch(this,
			FTouchId(TouchEvent.GetInputDeviceId(), static_cast<ETouchIndex::Type>(TouchEvent.GetPointerIndex())), ETouchType::Ended, CurCursorPos,
			/* Force */ 0.0f, TouchEvent.GetEventTimestamp());
		if (!bEventConsumed)
		{
			CurrentReplyState = FReply::Unhandled();
		}

		if (ClientPtr->GetMouseCaptureMode() == EMouseCaptureMode::CaptureDuringMouseDown)
		{
			CurrentReplyState.ReleaseMouseCapture();
		}
	}

	return CurrentReplyState;
}

FReply FSceneViewport::OnTouchForceChanged(const FGeometry& MyGeometry, const FPointerEvent& TouchEvent)
{
	// Start a new reply state
	CurrentReplyState = FReply::Handled();

	UpdateCachedCursorPos(MyGeometry, TouchEvent);
	UpdateCachedGeometry(MyGeometry);

PRAGMA_DISABLE_DEPRECATION_WARNINGS
	if (FViewportClient* ClientPtr = ViewportClient)
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	{
		// Switch to the viewport clients world before processing input
		FScopedConditionalWorldSwitcher WorldSwitcher(ClientPtr);

		const FVector2D TouchPosition = MyGeometry.AbsoluteToLocal(TouchEvent.GetScreenSpacePosition()) * MyGeometry.Scale;

		const bool bEventConsumed = ClientPtr->InputTouch(this,
			FTouchId(TouchEvent.GetInputDeviceId(), static_cast<ETouchIndex::Type>(TouchEvent.GetPointerIndex())), ETouchType::ForceChanged,
			TouchPosition, TouchEvent.GetTouchForce(), TouchEvent.GetEventTimestamp());
		if (!bEventConsumed)
		{
			CurrentReplyState = FReply::Unhandled();
		}
	}

	return CurrentReplyState;
}

FReply FSceneViewport::OnTouchFirstMove(const FGeometry& MyGeometry, const FPointerEvent& TouchEvent)
{
	// Start a new reply state
	CurrentReplyState = FReply::Handled();

	UpdateCachedCursorPos(MyGeometry, TouchEvent);
	UpdateCachedGeometry(MyGeometry);

PRAGMA_DISABLE_DEPRECATION_WARNINGS
	if (FViewportClient* ClientPtr = ViewportClient)
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	{
		// Switch to the viewport clients world before processing input
		FScopedConditionalWorldSwitcher WorldSwitcher(ClientPtr);

		const FVector2D TouchPosition = MyGeometry.AbsoluteToLocal(TouchEvent.GetScreenSpacePosition()) * MyGeometry.Scale;

		const bool bEventConsumed = ClientPtr->InputTouch(this,
			FTouchId(TouchEvent.GetInputDeviceId(), static_cast<ETouchIndex::Type>(TouchEvent.GetPointerIndex())), ETouchType::FirstMove,
			TouchPosition, TouchEvent.GetTouchForce(), TouchEvent.GetEventTimestamp());
		if (!bEventConsumed)
		{
			CurrentReplyState = FReply::Unhandled();
		}
	}

	return CurrentReplyState;
}

FReply FSceneViewport::OnTouchGesture( const FGeometry& MyGeometry, const FPointerEvent& GestureEvent )
{
	// Start a new reply state
	CurrentReplyState = FReply::Handled();

	UpdateCachedGeometry(MyGeometry);
	UpdateCachedCursorPos( MyGeometry, GestureEvent );

PRAGMA_DISABLE_DEPRECATION_WARNINGS
	if (FViewportClient* ClientPtr = ViewportClient)
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	{
		// Switch to the viewport clients world before processing input
		FScopedConditionalWorldSwitcher WorldSwitcher(ClientPtr);

		if (GestureEvent.GetGestureType() != EGestureEvent::LongPress)
		{
			FSlateApplication::Get().SetKeyboardFocus(ViewportWidget.Pin());
		}

		if(!ClientPtr->InputGesture(this, GestureEvent.GetInputDeviceId(), GestureEvent.GetGestureType(), GestureEvent.GetGesturePhase(), GestureEvent.GetGestureDelta(), GestureEvent.IsDirectionInvertedFromDevice(), GestureEvent.GetEventTimestamp()))
		{
			CurrentReplyState = FReply::Unhandled();
		}
	}

	return CurrentReplyState;
}

FReply FSceneViewport::OnMotionDetected( const FGeometry& MyGeometry, const FMotionEvent& MotionEvent )
{
	// Start a new reply state
	CurrentReplyState = FReply::Handled();

PRAGMA_DISABLE_DEPRECATION_WARNINGS
	if (FViewportClient* ClientPtr = ViewportClient)
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	{
		// Switch to the viewport clients world before processing input
		FScopedConditionalWorldSwitcher WorldSwitcher(ClientPtr);

		if (!ClientPtr->InputMotion( this, MotionEvent.GetInputDeviceId(), MotionEvent.GetTilt(), MotionEvent.GetRotationRate(), MotionEvent.GetGravity(), MotionEvent.GetAcceleration(), MotionEvent.GetEventTimestamp()))
		{
			CurrentReplyState = FReply::Unhandled(); 
		}
	}

	return CurrentReplyState;
}

FPopupMethodReply FSceneViewport::OnQueryPopupMethod() const
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	if (FViewportClient* ClientPtr = ViewportClient)
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	{
		return ClientPtr->OnQueryPopupMethod();
	}
	else
	{
		return FPopupMethodReply::Unhandled();
	}
}

bool FSceneViewport::HandleNavigation(const uint32 InUserIndex, TSharedPtr<SWidget> InDestination)
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	if (FViewportClient* ClientPtr = ViewportClient)
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	{
		return ClientPtr->HandleNavigation(InUserIndex, InDestination);
	}
	return false;
}

TOptional<bool> FSceneViewport::OnQueryShowFocus(const EFocusCause InFocusCause) const
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	if (FViewportClient* ClientPtr = ViewportClient)
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	{
		// Switch to the viewport clients world before processing input
		FScopedConditionalWorldSwitcher WorldSwitcher(ClientPtr);

		return ClientPtr->QueryShowFocus(InFocusCause);
	}

	return TOptional<bool>();
}

void FSceneViewport::OnFinishedPointerInput()
{
	ProcessAccumulatedPointerInput();
}

FReply FSceneViewport::OnKeyDown( const FGeometry& InGeometry, const FKeyEvent& InKeyEvent )
{
	// Start a new reply state
	CurrentReplyState = FReply::Handled(); 

	FKey Key = InKeyEvent.GetKey();
	if (Key.IsValid())
	{
		KeyStateMap.Add(Key, true);

		//@todo Slate Viewports: FWindowsViewport checks for Alt+Enter or F11 and toggles fullscreen.  Unknown if fullscreen via this method will be needed for slate viewports. 
PRAGMA_DISABLE_DEPRECATION_WARNINGS
		FViewportClient* ClientPtr = ViewportClient;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
		if (ClientPtr && GetSizeXY() != FIntPoint::ZeroValue)
		{
			// Switch to the viewport clients world before processing input
			FScopedConditionalWorldSwitcher WorldSwitcher(ClientPtr);

			if (!ClientPtr->InputKey(FInputKeyEventArgs(
				this, InKeyEvent.GetInputDeviceId(), Key, InKeyEvent.IsRepeat() ? IE_Repeat : IE_Pressed, 1.0f, false, InKeyEvent.GetEventTimestamp())))
			{
				CurrentReplyState = FReply::Unhandled();
			}
		}
	}
	else
	{
		CurrentReplyState = FReply::Unhandled();
	}
	return CurrentReplyState;
}

FReply FSceneViewport::OnKeyUp( const FGeometry& InGeometry, const FKeyEvent& InKeyEvent )
{
	// Start a new reply state
	CurrentReplyState = FReply::Handled(); 

	FKey Key = InKeyEvent.GetKey();
	if (Key.IsValid())
	{
		KeyStateMap.Add(Key, false);

PRAGMA_DISABLE_DEPRECATION_WARNINGS
		FViewportClient* ClientPtr = ViewportClient;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
		if (ClientPtr && GetSizeXY() != FIntPoint::ZeroValue)
		{
			// Switch to the viewport clients world before processing input
			FScopedConditionalWorldSwitcher WorldSwitcher(ClientPtr);

			if (!ClientPtr->InputKey(FInputKeyEventArgs(
				this, InKeyEvent.GetInputDeviceId(), Key, IE_Released, 1.0f, false, InKeyEvent.GetEventTimestamp())))
			{
				CurrentReplyState = FReply::Unhandled();
			}
		}
	}
	else
	{
		CurrentReplyState = FReply::Unhandled();
	}

	return CurrentReplyState;
}

FReply FSceneViewport::OnAnalogValueChanged(const FGeometry& MyGeometry, const FAnalogInputEvent& InAnalogInputEvent)
{
	// Start a new reply state
	CurrentReplyState = FReply::Handled();

	FKey Key = InAnalogInputEvent.GetKey();
	if (Key.IsValid())
	{
		KeyStateMap.Add(Key, true);
PRAGMA_DISABLE_DEPRECATION_WARNINGS
		if (FViewportClient* ClientPtr = ViewportClient)
PRAGMA_ENABLE_DEPRECATION_WARNINGS
		{
			// Switch to the viewport clients world before processing input
			FScopedConditionalWorldSwitcher WorldSwitcher(ClientPtr);

			FInputKeyEventArgs Args(
				this,
				InAnalogInputEvent.GetInputDeviceId(),
				Key,
				Key == EKeys::Gamepad_RightY ? -InAnalogInputEvent.GetAnalogValue() : InAnalogInputEvent.GetAnalogValue(),
				FApp::GetDeltaTime(),
				1,
				InAnalogInputEvent.GetEventTimestamp());

			if (!ClientPtr->InputAxis(Args))
			{
				CurrentReplyState = FReply::Unhandled();
			}
		}
	}
	else
	{
		CurrentReplyState = FReply::Unhandled();
	}

	return CurrentReplyState;
}

FReply FSceneViewport::OnKeyChar( const FGeometry& InGeometry, const FCharacterEvent& InCharacterEvent )
{
	// Start a new reply state
	CurrentReplyState = FReply::Handled(); 

PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FViewportClient* ClientPtr = ViewportClient;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	if (ClientPtr && GetSizeXY() != FIntPoint::ZeroValue)
	{
		// Switch to the viewport clients world before processing input
		FScopedConditionalWorldSwitcher WorldSwitcher(ClientPtr);

		if (!ClientPtr->InputChar(this, InCharacterEvent.GetUserIndex(), InCharacterEvent.GetCharacter()))
		{
			CurrentReplyState = FReply::Unhandled();
		}
	}
	return CurrentReplyState;
}

FReply FSceneViewport::OnFocusReceived(const FFocusEvent& InFocusEvent)
{
	CurrentReplyState = FReply::Handled(); 

	if ( InFocusEvent.GetUser() == FSlateApplication::Get().GetUserIndexForKeyboard() )
	{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
		FViewportClient* ClientPtr = ViewportClient;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
		if (ClientPtr)
		{
			FScopedConditionalWorldSwitcher WorldSwitcher(ClientPtr);
			ClientPtr->ReceivedFocus(this);
		}

		// Update key state mappings so that the the viewport modifier states are valid upon focus.
		const FModifierKeysState KeysState = FSlateApplication::Get().GetModifierKeys();
		KeyStateMap.Add(EKeys::LeftAlt, KeysState.IsLeftAltDown());
		KeyStateMap.Add(EKeys::RightAlt, KeysState.IsRightAltDown());
		KeyStateMap.Add(EKeys::LeftControl, KeysState.IsLeftControlDown());
		KeyStateMap.Add(EKeys::RightControl, KeysState.IsRightControlDown());
		KeyStateMap.Add(EKeys::LeftShift, KeysState.IsLeftShiftDown());
		KeyStateMap.Add(EKeys::RightShift, KeysState.IsRightShiftDown());
		KeyStateMap.Add(EKeys::LeftCommand, KeysState.IsLeftCommandDown());
		KeyStateMap.Add(EKeys::RightCommand, KeysState.IsRightCommandDown());


		if ( IsCurrentlyGameViewport() )
		{
			FSlateApplication& SlateApp = FSlateApplication::Get();

			const bool bPermanentCapture = (!GIsEditor || InFocusEvent.GetCause() == EFocusCause::Mouse)
				&& ClientPtr != nullptr
				&& (ClientPtr->GetMouseCaptureMode() == EMouseCaptureMode::CapturePermanently
					|| ClientPtr->GetMouseCaptureMode() == EMouseCaptureMode::CapturePermanently_IncludingInitialMouseDown);

			// if bPermanentCapture is true, then ViewportClient != nullptr, so it's ok to dereference it.  But the permanent capture must be tested first.
			if (SlateApp.IsActive() && bPermanentCapture && !ClientPtr->IgnoreInput())
			{
				TSharedRef<SViewport> ViewportWidgetRef = ViewportWidget.Pin().ToSharedRef();

				FWidgetPath PathToWidget;
				SlateApp.GeneratePathToWidgetUnchecked(ViewportWidgetRef, PathToWidget);

				return AcquireFocusAndCapture(GetSizeXY() / 2, EFocusCause::Mouse);
			}
		}
	}

	return CurrentReplyState;
}

void FSceneViewport::OnFocusLost( const FFocusEvent& InFocusEvent )
{
	// If the focus loss event isn't the for the primary 'keyboard' user, don't worry about it.
	if ( InFocusEvent.GetUser() != FSlateApplication::Get().GetUserIndexForKeyboard() )
	{
		return;
	}

	CachedUnboundedCursorPos.Reset();
	bShouldCaptureMouseOnActivate = false;
	bCursorHiddenDueToCapture = false;
	KeyStateMap.Empty();
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	if (FViewportClient* ClientPtr = ViewportClient)
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	{
		FScopedConditionalWorldSwitcher WorldSwitcher(ClientPtr);
		ClientPtr->LostFocus(this);
	}
}

void FSceneViewport::OnViewportClosed()
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	if (FViewportClient* ClientPtr = ViewportClient)
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	{
		FScopedConditionalWorldSwitcher WorldSwitcher(ClientPtr);
		ClientPtr->CloseRequested(this);
	}
}

FReply FSceneViewport::OnRequestWindowClose()
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FViewportClient* ClientPtr = ViewportClient;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	return ClientPtr && !ClientPtr->WindowCloseRequested() ? FReply::Handled() : FReply::Unhandled();
}

TWeakPtr<SWidget> FSceneViewport::GetWidget()
{
	return GetViewportWidget();
}

FReply FSceneViewport::OnViewportActivated(const FWindowActivateEvent& InActivateEvent)
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	if (FViewportClient* ClientPtr = ViewportClient)
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	{
		FScopedConditionalWorldSwitcher WorldSwitcher(ClientPtr);
		ClientPtr->Activated(this, InActivateEvent);

		// Determine if we're in permanent capture mode.  This cannot be cached as part of bShouldCaptureMouseOnActivate because it could change between window activate and deactivate
		const bool bPermanentCapture = ClientPtr->IsInPermanentCapture();

		// If we are activating and had Mouse Capture on deactivate then we should get focus again
		// It's important to note in the case of:
		//    InActivateEvent.ActivationType == FWindowActivateEvent::EA_ActivateByMouse
		// we do NOT acquire focus the reasoning is that the click itself will give us a chance on Mouse down to get capture.
		// This also means we don't go and grab capture in situations like:
		//    - the user clicked on the application header
		//    - the user clicked on some UI
		//    - the user clicked in our window but not an area our viewport covers.
		if (InActivateEvent.GetActivationType() == FWindowActivateEvent::EA_Activate && (bShouldCaptureMouseOnActivate || bPermanentCapture))
		{
			return AcquireFocusAndCapture(GetSizeXY() / 2, EFocusCause::WindowActivate);
		}
	}

	return FReply::Unhandled();
}

EDisplayColorGamut FSceneViewport::GetDisplayColorGamut() const
{
	return DisplayColorGamut;
}

EDisplayOutputFormat FSceneViewport::GetDisplayOutputFormat() const
{
	return DisplayOutputFormat;
}

bool FSceneViewport::GetSceneHDREnabled() const
{
	return bHDRViewport;
}

float FSceneViewport::GetMinimumLuminanceInNits() const
{
	return MinimumLuminanceInNits;
}

float FSceneViewport::GetMaximumLuminanceInNits() const
{
	return MaximumLuminanceInNits;
}

float FSceneViewport::GetMaximumFullFrameLuminanceInNits() const
{
	return MaximumFullFrameLuminanceInNits;
}

float FSceneViewport::GetHDRPaperWhiteInNits() const
{
	return HDRPaperWhiteInNits;
}

const UE::Color::FColorSpace& FSceneViewport::GetLimitingColorSpace() const
{
	return LimitingColorSpace;
}

ESlateViewportDynamicRange FSceneViewport::GetViewportDynamicRange() const
{
	return bHDRViewport ? ESlateViewportDynamicRange::HDR : ESlateViewportDynamicRange::SDR;
}

void FSceneViewport::OnViewportDeactivated(const FWindowActivateEvent& InActivateEvent)
{
	// We backup if we have capture for us on activation, however we also maintain "true" if it's already true!
	// The reasoning behind maintaining "true" is that if the viewport is activated, 
	// however doesn't reclaim capture we want to claim capture next time we activate unless something else gets focus.
	// So we reset bHadMouseCaptureOnDeactivate in AcquireFocusAndCapture() and in OnFocusLost()
	//
	// This is not ideal, however the better fix probably requires that slate fundamentally chance when it "activates" a window or maybe just the viewport
	// Which there simply doesn't exist the right hooks currently.
	//
	// This fixes the case where the application is deactivated, then the user click on the windows header
	// this activates the window but we do not capture the mouse, then the User Alt-Tabs to the application.
	// We properly acquire capture because we maintained the "true" through the activation where nothing was focuses
	bShouldCaptureMouseOnActivate = !GIsEditor  && (bShouldCaptureMouseOnActivate || HasMouseCapture());

	KeyStateMap.Empty();
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	if (FViewportClient* ClientPtr = ViewportClient)
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	{
		FScopedConditionalWorldSwitcher WorldSwitcher(ClientPtr);
		ClientPtr->Deactivated(this, InActivateEvent);
	}
}


bool FSceneViewport::IsCameraCut() const
{
	return bCameraCut;
}


FSlateShaderResource* FSceneViewport::GetViewportRenderTargetTexture() const
{
	if (IsInRenderingThread())
	{
		return RenderThreadSlateTexture;
	}
	return NumBufferedFrames ? BufferedFrames[BufferIndex_GT].SlateHandle : nullptr;
}

bool FSceneViewport::IsStereoscopic3D() const
{
	return GEngine->IsStereoscopic3D(this);
}

void FSceneViewport::SetDebugCanvas(TSharedPtr<SDebugCanvas> InDebugCanvas)
{
	DebugCanvas = InDebugCanvas;
}

void FSceneViewport::PaintDebugCanvas(const FGeometry& AllottedGeometry, FSlateWindowElementList& OutDrawElements, int32 LayerId) const
{
	if (DebugCanvasDrawer->GetGameThreadDebugCanvas() && DebugCanvasDrawer->GetGameThreadDebugCanvas()->HasBatchesToRender())
	{
		// Cannot pass negative canvas positions
		float CanvasMinX = FMath::Max(0.0f, AllottedGeometry.AbsolutePosition.X);
		float CanvasMinY = FMath::Max(0.0f, AllottedGeometry.AbsolutePosition.Y);
		FIntRect CanvasRect(
			FMath::RoundToInt(CanvasMinX),
			FMath::RoundToInt(CanvasMinY),
			FMath::RoundToInt(CanvasMinX + AllottedGeometry.GetLocalSize().X * AllottedGeometry.Scale),
			FMath::RoundToInt(CanvasMinY + AllottedGeometry.GetLocalSize().Y * AllottedGeometry.Scale));

		DebugCanvasDrawer->BeginRenderingCanvas(CanvasRect);

		FSlateDrawElement::MakeCustom(OutDrawElements, LayerId, DebugCanvasDrawer);
	}
}

static void EnsureValidFullscreenResolution(const FMonitorInfo& Monitor, uint32& NewWindowSizeX, uint32& NewWindowSizeY)
{
	FScreenResolutionArray ResArray;
	RHIGetAvailableResolutionsForDisplay(ResArray, true, Monitor.NativeHandle);
	if (ResArray.IsEmpty())
	{
		ResArray.Emplace(Monitor.DisplayRect.Right - Monitor.DisplayRect.Left, Monitor.DisplayRect.Bottom - Monitor.DisplayRect.Top);
	}
	bool bFound = false;
	for (FScreenResolutionRHI ScreenResolution : ResArray)
	{
		bFound = ScreenResolution.Width == NewWindowSizeX && ScreenResolution.Height == NewWindowSizeY;
		if (bFound)
		{
			break;
		}
	}
	if (!bFound && !ResArray.IsEmpty())
	{
		NewWindowSizeX = ResArray[0].Width;
		NewWindowSizeY = ResArray[0].Height;
	}
}

void FSceneViewport::ResizeFrame(uint32 NewWindowSizeX, uint32 NewWindowSizeY, EWindowMode::Type NewWindowMode)
{
	// Resizing the window directly is only supported in the game
	if( FApp::IsGame() && FApp::CanEverRender() && NewWindowSizeX > 0 && NewWindowSizeY > 0 )
	{
		if (TSharedPtr<SWindow> WindowToResize = FindWindow())
		{
			NewWindowMode = GetWindowModeType(NewWindowMode);

			const FVector2D OldWindowPos = WindowToResize->GetPositionInScreen();
			const FVector2D OldWindowSize = WindowToResize->GetClientSizeInScreen();
			const EWindowMode::Type OldWindowMode = WindowToResize->GetWindowMode();

			// Check if switching to exclusive fullscreen mode on a different monitor.
			static bool bAllowRecursion = true;
			if (FPlatformProperties::SupportsWindowedMode() && (NewWindowMode == EWindowMode::Fullscreen) && bAllowRecursion)
			{
				int32 NewLeft = 0;
				int32 NewTop = 0;
				const FDisplayMetrics& DisplayMetrics = FSlateApplication::Get().GetCachedDisplayMetricsByRef();
				if (DisplayMetrics.MonitorInfo.IsValidIndex(MonitorIndexForResize))
				{
					const FMonitorInfo& Monitor = DisplayMetrics.MonitorInfo[MonitorIndexForResize];
					const FPlatformRect& DisplayRect = DisplayMetrics.MonitorInfo[MonitorIndexForResize].DisplayRect;
					NewLeft = DisplayRect.Left;
					NewTop = DisplayRect.Top;
					EnsureValidFullscreenResolution(Monitor, NewWindowSizeX, NewWindowSizeY);
				}
				else
				{
					for (const FMonitorInfo& Monitor : DisplayMetrics.MonitorInfo)
					{
						if (Monitor.bIsPrimary)
						{
							EnsureValidFullscreenResolution(Monitor, NewWindowSizeX, NewWindowSizeY);
							break;
						}
					}
				}

				const FSlateRect& OldFullScreenInfo = WindowToResize->GetFullScreenInfo();
				if ((OldFullScreenInfo.Left != NewLeft) || (OldFullScreenInfo.Top != NewTop))
				{
					bAllowRecursion = false;
					// Avoid switching to exclusive fullscreen on the wrong monitor.
					ResizeFrame(NewWindowSizeX, NewWindowSizeY, EWindowMode::WindowedFullscreen);
					ResizeFrame(NewWindowSizeX, NewWindowSizeY, EWindowMode::Fullscreen);
					bAllowRecursion = true;
					return;
				}
			}

			// Set the new window mode first to ensure that the work area size is correct (fullscreen windows can affect this)
			if (NewWindowMode != OldWindowMode)
			{
				WindowToResize->SetWindowMode(NewWindowMode);
				WindowMode = NewWindowMode;
			}

			TOptional<FVector2D> NewWindowPos;
			FVector2D NewWindowSize(NewWindowSizeX, NewWindowSizeY);

			// Only adjust window size if not in off-screen rendering mode, because off-screen rendering skips rendering to screen and uses custom size.
			if (!FSlateApplication::Get().IsRenderingOffScreen())
			{
				const FDisplayMetrics& DisplayMetrics = FSlateApplication::Get().GetCachedDisplayMetricsByRef();
				const bool bValidMonitorIndex = DisplayMetrics.MonitorInfo.IsValidIndex(MonitorIndexForResize);
				FSlateRect BestWorkArea;
				if (bValidMonitorIndex)
				{
					const FPlatformRect& WorkArea = DisplayMetrics.MonitorInfo[MonitorIndexForResize].WorkArea;
					BestWorkArea = FSlateRect(WorkArea.Left, WorkArea.Top, WorkArea.Right, WorkArea.Bottom);
				}
				else
				{
					BestWorkArea = FSlateApplication::Get().GetWorkArea(FSlateRect::FromPointAndExtent(OldWindowPos, OldWindowSize));
				}

				// A switch to window mode should position the window to be in the center of the work-area (we don't do this if we were already in window mode to allow the user to move the window)
				// Fullscreen modes should position the window to the top-left of the monitor.
				// If we're going into windowed fullscreen mode, we always want the window to fill the entire screen.
				// When we calculate the scene view, we'll check the fullscreen mode and configure the screen percentage
				// scaling so we actual render to the resolution we've been asked for.
				if (NewWindowMode == EWindowMode::Windowed)
				{
					bool bKeepOldWindowPos = (OldWindowMode == EWindowMode::Windowed) && (NewWindowSize == OldWindowSize);
					if (bKeepOldWindowPos && bValidMonitorIndex)
					{
						const FPlatformRect& DisplayRect = DisplayMetrics.MonitorInfo[MonitorIndexForResize].DisplayRect;
						const FSlateRect FullScreenInfo = WindowToResize->GetFullScreenInfo();
						bKeepOldWindowPos = (DisplayRect.Left == FullScreenInfo.Left) && (DisplayRect.Top == FullScreenInfo.Top);
					}

					if (bKeepOldWindowPos)
					{
						NewWindowPos.Reset();
					}
					else
					{
						const FVector2D BestWorkAreaTopLeft = BestWorkArea.GetTopLeft();
						const FVector2D BestWorkAreaSize = BestWorkArea.GetSize();

						FVector2D CenteredWindowPos = BestWorkAreaTopLeft;

						if (NewWindowSize.X < BestWorkAreaSize.X)
						{
							CenteredWindowPos.X += (BestWorkAreaSize.X - NewWindowSize.X) * 0.5f;
						}

						if (NewWindowSize.Y < BestWorkAreaSize.Y)
						{
							CenteredWindowPos.Y += (BestWorkAreaSize.Y - NewWindowSize.Y) * 0.5f;
						}

						NewWindowPos = CenteredWindowPos;
					}
				}
				else if (!DisplayMetrics.MonitorInfo.IsEmpty())
				{
					FPlatformRect DisplayRect;
					if (bValidMonitorIndex)
					{
						DisplayRect = DisplayMetrics.MonitorInfo[MonitorIndexForResize].DisplayRect;
					}
					else
					{
						const FSlateRect& FullScreenInfo = WindowToResize->GetFullScreenInfo();
						DisplayRect = FPlatformRect(FullScreenInfo.Left, FullScreenInfo.Top, FullScreenInfo.Right, FullScreenInfo.Bottom);
					}

					NewWindowPos = FVector2D(DisplayRect.Left, DisplayRect.Top);

					if (NewWindowMode == EWindowMode::WindowedFullscreen)
					{
						NewWindowSize.X = DisplayRect.Right - DisplayRect.Left;
						NewWindowSize.Y = DisplayRect.Bottom - DisplayRect.Top;
					}
				}
				else
				{
					NewWindowPos = FVector2D(0.0f, 0.0f);

					if (NewWindowMode == EWindowMode::WindowedFullscreen)
					{
						NewWindowSize.X = DisplayMetrics.PrimaryDisplayWidth;
						NewWindowSize.Y = DisplayMetrics.PrimaryDisplayHeight;
					}
				}

#if !PLATFORM_MAC
				IHeadMountedDisplay::MonitorInfo MonitorInfo;
				if (GEngine->XRSystem.IsValid() && GEngine->XRSystem->GetHMDDevice() && GEngine->XRSystem->GetHMDDevice()->GetHMDMonitorInfo(MonitorInfo))
				{
					// Desktop platfoms generally don't check the monitor resolution.
					if (MonitorInfo.DesktopX > 0 || MonitorInfo.DesktopY > 0 || (MonitorInfo.bShouldTestResolution && (MonitorInfo.ResolutionX > 0 || MonitorInfo.ResolutionY > 0)))
					{
						NewWindowSize.X = MonitorInfo.ResolutionX;
						NewWindowSize.Y = MonitorInfo.ResolutionY;
						NewWindowPos = FVector2D(MonitorInfo.DesktopX, MonitorInfo.DesktopY);
					}
				}
#endif
			}
			else
			{
				NewWindowPos = FVector2D(0.0f, 0.0f);
			}

			// Resize window
			const bool bSizeChanged = NewWindowSize != OldWindowSize;
			const bool bPositionChanged = NewWindowPos.IsSet() && NewWindowPos != OldWindowPos;
			const bool bModeChanged = NewWindowMode != OldWindowMode;
			if (bSizeChanged || bPositionChanged || bModeChanged)
			{
				if (CurrentReplyState.ShouldReleaseMouseLock())
				{
					LockMouseToViewport(false);
				}

				if (bModeChanged || (bSizeChanged && bPositionChanged))
				{
					WindowToResize->ReshapeWindow(NewWindowPos.GetValue(), NewWindowSize);
				}
				else if (bSizeChanged)
				{
					WindowToResize->Resize(NewWindowSize);
				}
				else
				{
					WindowToResize->MoveWindowTo(NewWindowPos.GetValue());
				}
			}

			// Resize viewport
			FVector2D ViewportSize = WindowToResize->GetWindowSizeFromClientSize(FVector2D(SizeX, SizeY));
			FVector2D NewViewportSize = WindowToResize->GetViewportSize();

			// Resize backbuffer
			FVector2D BackBufferSize = WindowToResize->IsMirrorWindow() ? OldWindowSize : ViewportSize;
			FVector2D NewBackbufferSize = WindowToResize->IsMirrorWindow() ? NewWindowSize : NewViewportSize;

			if (NewViewportSize != ViewportSize || NewWindowMode != OldWindowMode)
			{
				FSlateApplicationBase::Get().GetRenderer()->UpdateFullscreenState(WindowToResize.ToSharedRef(), NewBackbufferSize.X, NewBackbufferSize.Y);
				ResizeViewport(NewViewportSize.X, NewViewportSize.Y, NewWindowMode);
			}

			if(NewBackbufferSize != BackBufferSize)
			{
				FSlateApplicationBase::Get().GetRenderer()->UpdateFullscreenState(WindowToResize.ToSharedRef(), NewBackbufferSize.X, NewBackbufferSize.Y);
			}

			UCanvas::UpdateAllCanvasSafeZoneData();
		}
	}
}

bool FSceneViewport::HasFixedSize() const
{
	return bForceViewportSize;
}

void FSceneViewport::SetFixedViewportSize(uint32 NewViewportSizeX, uint32 NewViewportSizeY)
{
	if (ViewportWidget.IsValid())
	{
		if (NewViewportSizeX > 0 && NewViewportSizeY > 0)
		{
			bForceViewportSize = true;
			if (TSharedPtr<SWindow> Window = FindWindow())
			{
				ResizeViewport(NewViewportSizeX, NewViewportSizeY, Window->GetWindowMode());
			}
		}
		else
		{
			bForceViewportSize = false;
			if (TSharedPtr<SWindow> Window = FindWindow())
			{
				Window->Invalidate(EInvalidateWidget::PaintAndVolatility);
			}
		}
	}
}

void FSceneViewport::SetViewportSize(uint32 NewViewportSizeX, uint32 NewViewportSizeY)
{
	if (TSharedPtr<SWindow> Window = FindWindow())
	{
		Window->SetIndependentViewportSize(FVector2D(NewViewportSizeX, NewViewportSizeY));
		const FVector2D vp = Window->IsMirrorWindow() ? Window->GetSizeInScreen() : Window->GetViewportSize();
		FSlateApplicationBase::Get().GetRenderer()->UpdateFullscreenState(Window.ToSharedRef(), vp.X, vp.Y);
		ResizeViewport(NewViewportSizeX, NewViewportSizeY, Window->GetWindowMode());
	}
}

TSharedPtr<SWindow> FSceneViewport::FindWindow() const
{
	TSharedPtr<SWindow> Window;
	if (ViewportWidget.IsValid())
	{
		TSharedPtr<SViewport> PinnedViewportWidget = ViewportWidget.Pin();
		Window = FSlateApplication::Get().FindWidgetWindow(PinnedViewportWidget.ToSharedRef());
	}

	if (!Window.IsValid())
	{
		// If we can't find the window via the ViewportWidget, check the game engine instance instead.
		if (UGameEngine* GameEngine = Cast<UGameEngine>(GEngine))
		{
			if (this == GameEngine->SceneViewport.Get())
			{
				Window = GameEngine->GameViewportWindow.Pin();
			}
		}
	}

	return Window;
}

bool FSceneViewport::IsStereoRenderingAllowed() const
{
	if (ViewportWidget.IsValid())
	{
		return ViewportWidget.Pin()->IsStereoRenderingAllowed();
	}
	return false;
}

void FSceneViewport::ResizeViewport(uint32 NewSizeX, uint32 NewSizeY, EWindowMode::Type NewWindowMode)
{
	// Do not resize if the viewport is an invalid size or our UI should be responsive
	if( NewSizeX > 0 && NewSizeY > 0)
	{
		uint32 MaxSize = GetMax2DTextureDimension();
		// When the size is larger than the biggest texture possible, we clamp it to the max size but still preserve aspect ratio
		if (NewSizeX > MaxSize || NewSizeY > MaxSize)
		{
			float Ratio = (float)NewSizeX / (float)NewSizeY;
			if (NewSizeX > NewSizeY)
			{
				NewSizeX = MaxSize;
				NewSizeY = NewSizeX / Ratio;
			}
			else
			{
				NewSizeY = MaxSize;
				NewSizeX = NewSizeY * Ratio;
			}
		}

		const TCHAR* WindowModeName;
		switch (NewWindowMode)
		{
		case EWindowMode::Fullscreen: WindowModeName = TEXT("Fullscreen"); break;
		case EWindowMode::WindowedFullscreen: WindowModeName = TEXT("WindowedFullscreen"); break;
		case EWindowMode::Windowed: WindowModeName = TEXT("Windowed"); break;
		default: WindowModeName = TEXT("INVALID"); break;
		}

		UE_LOGF(LogViewport, Verbose, "Scene viewport resized to %dx%d, mode %ls.", NewSizeX, NewSizeY, WindowModeName);

		bIsResizing = true;

		UpdateViewportRHI(false, NewSizeX, NewSizeY, NewWindowMode, PF_Unknown);
		FCoreDelegates::OnSafeFrameChangedEvent.Broadcast();

PRAGMA_DISABLE_DEPRECATION_WARNINGS
		if (FViewportClient* ClientPtr = ViewportClient)
PRAGMA_ENABLE_DEPRECATION_WARNINGS
		{
			// Invalidate, then redraw immediately so the user isn't left looking at an empty black viewport
			// as they continue to resize the window.
			Invalidate();

			if (ClientPtr->GetWorld())
			{
				Draw();
			}
		}

		//if we have a delegate, fire it off
		if(FApp::IsGame() && OnSceneViewportResizeDel.IsBound())
		{
			OnSceneViewportResizeDel.Execute(FVector2D(NewSizeX, NewSizeY));
		}

		bIsResizing = false;
	}
}

void FSceneViewport::InvalidateDisplay()
{
	// Dirty the viewport.  It will be redrawn next time the editor ticks.
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	if (FViewportClient* ClientPtr = ViewportClient)
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	{
		ClientPtr->RedrawRequested(this);
	}
}

void FSceneViewport::DeferInvalidateHitProxy()
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	if (FViewportClient* ClientPtr = ViewportClient)
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	{
		ClientPtr->RequestInvalidateHitProxy(this);
	}
}

FCanvas* FSceneViewport::GetDebugCanvas()
{
	return DebugCanvasDrawer->GetGameThreadDebugCanvas();
}

float FSceneViewport::GetDisplayGamma() const
{
	if (ViewportGammaOverride.IsSet())
	{
		return ViewportGammaOverride.GetValue();
	}
	return	FViewport::GetDisplayGamma();
}

int32 FSceneViewport::GetCurrentMonitorIndex() const
{
	int32 MonitorIndex = -1;

	const FDisplayMetrics& DisplayMetrics = FSlateApplication::Get().GetCachedDisplayMetricsByRef();
	const int32 Num = DisplayMetrics.MonitorInfo.Num();
	TSharedPtr<SWidget> PinnedViewport = ViewportWidget.Pin();
	if (PinnedViewport.IsValid())
	{
		if (TSharedPtr<SWindow> Window = FindWindow())
		{
			const FSlateRect FullScreenInfo = Window->GetFullScreenInfo();
			for (int32 Index = 0; Index < Num; ++Index)
			{
				const FPlatformRect& DisplayRect = DisplayMetrics.MonitorInfo[Index].DisplayRect;
				if ((DisplayRect.Left == FullScreenInfo.Left) && (DisplayRect.Top == FullScreenInfo.Top))
				{
					MonitorIndex = Index;
					break;
				}
			}
		}
	}

	return MonitorIndex;
}

bool FSceneViewport::IsExclusiveFullscreenSupported() const
{
	bool bSupported = true;

	if (TSharedPtr<SWindow> Window = FindWindow())
	{
		bSupported = Window->GetNativeWindow()->IsFullscreenSupported();
	}

	return bSupported;
}

void FSceneViewport::SetViewportClient(const TStrongPtrVariant<FViewportClient>& InViewportClient)
{
	FViewport::SetViewportClient(InViewportClient);

	if (FViewportClient* const ClientPtr = InViewportClient.Get())
	{
		bShouldCaptureMouseOnActivate = ClientPtr->CaptureMouseOnLaunch();
	}
	else
	{
		bShouldCaptureMouseOnActivate = true;
	}
}

const FTextureRHIRef& FSceneViewport::GetRenderTargetTexture() const
{
	if (IsInRenderingThread())
	{
		return RenderTargetTextureRenderThreadRHI;
	}
	return 	RenderTargetTextureRHI;
}

void FSceneViewport::SetRenderTargetTextureRenderThread(FRHITexture* InRenderTargetTexture)
{
	check(IsInRenderingThread());
	RenderTargetTextureRenderThreadRHI = InRenderTargetTexture;
	if (InRenderTargetTexture)
	{
		RenderThreadSlateTexture->SetRHIRef(InRenderTargetTexture, InRenderTargetTexture->GetSizeX(), InRenderTargetTexture->GetSizeY());
	}
	else
	{
		RenderThreadSlateTexture->SetRHIRef(nullptr, 0, 0);
	}
}

void FSceneViewport::SetInitialSize(FIntPoint InitialSizeXY)
{
	// Initial size only works if the viewport has not yet been resized
	if (GetSizeXY() == FIntPoint::ZeroValue)
	{
		UpdateViewportRHI(false, InitialSizeXY.X, InitialSizeXY.Y, EWindowMode::Windowed, PF_Unknown);
	}
}

void FSceneViewport::UpdateViewportRHI(bool bDestroyed, uint32 NewSizeX, uint32 NewSizeY, EWindowMode::Type NewWindowMode, EPixelFormat PreferredPixelFormat)
{
	check(IsInGameThread());

	// Release the viewport's resources.
	if (DebugCanvasDrawer)
	{
		DebugCanvasDrawer->ReleaseInternalTexture();
	}
	BeginReleaseResource(this);
	FlushRenderingCommands();

	// Update the viewport attributes.
	// This is done AFTER the command flush done by UpdateViewportRHI, to avoid disrupting rendering thread accesses to the old viewport size.
	SizeX = NewSizeX;
	SizeY = NewSizeY;
	WindowMode = NewWindowMode;

	if (!bDestroyed)
	{
		TSharedPtr<SWidget> PinnedViewport = ViewportWidget.Pin();
		void* OSWindow = nullptr;
		FVector2D WindowTopLeft = FVector2D(0.0f, 0.0f);
		FVector2D WindowBottomRight = FVector2D(0.0f, 0.0f);

		if (PinnedViewport.IsValid())
		{
			if (TSharedPtr<SWindow> Window = FindWindow())
			{
				OSWindow = Window->GetNativeWindow()->GetOSWindowHandle();
				WindowTopLeft = Window->GetPositionInScreen();
				WindowBottomRight = Window->GetPositionInScreen() + Window->GetSizeInScreen();
			}
		}

		FHDRMetaData OutHDRMetaData;
		ComputeSceneViewportHDRMetaData(OutHDRMetaData, WindowTopLeft, WindowBottomRight, OSWindow, IsStereoRenderingAllowed());
		DisplayOutputFormat = OutHDRMetaData.DisplayOutputFormat;
		DisplayColorGamut = OutHDRMetaData.DisplayColorGamut;
		MinimumLuminanceInNits = OutHDRMetaData.MinimumLuminanceInNits;
		MaximumLuminanceInNits = OutHDRMetaData.MaximumLuminanceInNits;
		MaximumFullFrameLuminanceInNits = OutHDRMetaData.MaximumFullFrameLuminanceInNits;
		HDRPaperWhiteInNits = OutHDRMetaData.HDRPaperWhiteInNits;
		LimitingColorSpace = OutHDRMetaData.LimitingColorSpace;
		bHDRViewport = OutHDRMetaData.bHDRSupported;

		BeginInitResource(this);

		FRenderCommandFence InitResourceFence;
		InitResourceFence.BeginFence();
		InitResourceFence.Wait();

		FSlateRenderer* Renderer = FSlateApplication::Get().GetRenderer();

		if (PinnedViewport.IsValid())
		{
			if (UseSeparateRenderTarget())
			{
				uint32 TexSizeX = SizeX, TexSizeY = SizeY;
				if (IStereoRenderTargetManager* const StereoRenderTargetManager = RetrieveStereoRenderTargetManager(IsStereoRenderingAllowed()))
				{
					StereoRenderTargetManager->CalculateRenderTargetSize(*this, TexSizeX, TexSizeY);
				}
				RTTSize = FIntPoint(TexSizeX, TexSizeY);
			}
		}

		if (!UseSeparateRenderTarget())
		{
			TSharedPtr<SWindow> Window = FindWindow();
			Renderer->UpdateFullscreenState(Window.ToSharedRef(), NewSizeX, NewSizeY);
		}

		ViewportResizedEvent.Broadcast(this, 0);
	}
	else
	{
		TArray<FSlateRenderTargetRHI*> SlateHandlesToDelete;

		SlateHandlesToDelete.Emplace(RenderThreadSlateTexture);
		RenderThreadSlateTexture = nullptr;

		for (FFrameData& FrameData : BufferedFrames)
		{
			SlateHandlesToDelete.Emplace(FrameData.SlateHandle);
			FrameData.SlateHandle = nullptr;
		}

		// Enqueue a render command to delete the slate handles. They must be deleted on the render thread after the resource is released.
		ENQUEUE_RENDER_COMMAND(DeleteSlateRenderTarget)(
			[SlateHandlesToDelete = MoveTemp(SlateHandlesToDelete)](FRHICommandListImmediate& RHICmdList)
			{
				for (FSlateRenderTargetRHI* SlateHandle : SlateHandlesToDelete)
				{
					delete SlateHandle;
				}
			});

	}
}

void FSceneViewport::EnqueueBeginRenderFrame(const bool bShouldPresent)
{
	check(IsInGameThread());
	checkf(SizeX > 0 && SizeY > 0, TEXT("FSceneViewport::EnqueueBeginRenderFrame called before calling FSceneViewport::ResizeViewport, this is unsafe and will cause crashes."));

	const bool bStereoRenderingAvailable = GEngine->StereoRenderingDevice.IsValid() && IsStereoRenderingAllowed();
	const bool bStereoRenderingEnabled = bStereoRenderingAvailable && GEngine->StereoRenderingDevice->IsStereoEnabled();

	IStereoRenderTargetManager* StereoRenderTargetManager = bStereoRenderingAvailable ? GEngine->StereoRenderingDevice->GetRenderTargetManager() : nullptr;

	BufferIndex_GT = NextBufferIndex_GT;
	NextBufferIndex_GT = NumBufferedFrames ? (BufferIndex_GT + 1) % NumBufferedFrames : 0;

	// check if we need to reallocate rendertarget for HMD and update HMD rendering viewport 
	if (bStereoRenderingAvailable)
	{
		bool bHMDWantsSeparateRenderTarget = StereoRenderTargetManager ? StereoRenderTargetManager->ShouldUseSeparateRenderTarget() : false;
		if (bHMDWantsSeparateRenderTarget != bForceSeparateRenderTarget ||
			(bHMDWantsSeparateRenderTarget && StereoRenderTargetManager->NeedReAllocateViewportRenderTarget(*this)))
		{
			// This will cause RT to be allocated (or freed)
			bForceSeparateRenderTarget = bHMDWantsSeparateRenderTarget;
			UpdateViewportRHI(false, SizeX, SizeY, WindowMode, PF_Unknown);
		}

		if (bHMDWantsSeparateRenderTarget)
		{
			// We need to acquire a buffered texture from either the new RT or the existing one
			int32 TextureIndex = StereoRenderTargetManager->AcquireColorTexture();
			BufferIndex_GT = TextureIndex < 0 ? BufferIndex_GT : TextureIndex;

			StereoRenderTargetManager->AcquireDepthTexture();
		}
	}

	ISlateViewportProvider* ViewportProvider = nullptr;
	if (NumBufferedFrames && BufferedFrames[BufferIndex_GT].RenderTarget)
	{
		RenderTargetTextureRHI = BufferedFrames[BufferIndex_GT].RenderTarget;
	}
	else
	{
		TSharedPtr<SWindow> Window = FindWindow();
		ViewportProvider = FSlateApplication::Get().GetRenderer()->GetViewportProvider(*Window);
	}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
	if (ViewportClient)
	{
		DebugCanvasDrawer->InitDebugCanvas(ViewportClient, ViewportClient->GetWorld());
	}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

	// Set the render target visible to the render thread. Must come before any render thread frame handling.
	ENQUEUE_RENDER_COMMAND(SetRenderThreadViewportTarget)([
		  NewRenderTargetTexture = RenderTargetTextureRHI
		, NewTargetIndex = BufferIndex_GT
		, bShouldPresent
		, ViewportProvider
		, this
		](FRHICommandListImmediate& RHICmdList)
	{
		BufferIndex_RT = NewTargetIndex;
		FRHITexture* Target = NewRenderTargetTexture;

		if (ViewportProvider)
		{
			Target = ViewportProvider->GetBackBufferResource();
		}

		check(Target);
		SetRenderTargetTextureRenderThread(Target);
	});
}

void FSceneViewport::EnqueueEndRenderFrame(const bool bLockToVsync, const bool bShouldPresent)
{
	bool bPresent = (PresentAndStopMovieDelay <= 0)
		? bShouldPresent
		: false;

	ENQUEUE_RENDER_COMMAND(EndDrawingCommand, LoopTick)([this, bLockToVsync, bPresent](FRHICommandListImmediate& RHICmdList)
	{
		if (UseSeparateRenderTarget())
		{
			if (BufferedFrames[BufferIndex_RT].SlateHandle)
			{
				RHICmdList.Transition(FRHITransitionInfo(RenderTargetTextureRenderThreadRHI, ERHIAccess::Unknown, ERHIAccess::SRVMask));
			}
		}
		else
		{
			// Note: this releases our reference but does not release the resource as it is owned by slate (this is intended)
			RenderTargetTextureRenderThreadRHI.SafeRelease();
			RenderThreadSlateTexture->SetRHIRef(nullptr, 0, 0);
		}
	});

	// Invalidate the debug canvas after rendering is complete if the debug canvas has elements
	if (DebugCanvasDrawer->GetGameThreadDebugCanvas() && DebugCanvasDrawer->GetGameThreadDebugCanvas()->HasBatchesToRender() && DebugCanvas.IsValid())
	{
		DebugCanvas.Pin()->Invalidate(EInvalidateWidget::Paint);
	}
}

void FSceneViewport::Tick( const FGeometry& AllottedGeometry, double InCurrentTime, float DeltaTime )
{
	UpdateCachedGeometry(AllottedGeometry);
	ProcessInput( DeltaTime );
}

void FSceneViewport::OnPlayWorldViewportSwapped( const FSceneViewport& OtherViewport )
{
	// Play world viewports should always be the same size.  Resize to other viewports size
	if( GetSizeXY() != OtherViewport.GetSizeXY() )
	{
		// Switch to the viewport clients world before processing input
PRAGMA_DISABLE_DEPRECATION_WARNINGS
		FScopedConditionalWorldSwitcher WorldSwitcher(ViewportClient);
PRAGMA_ENABLE_DEPRECATION_WARNINGS

		UpdateViewportRHI( false, OtherViewport.GetSizeXY().X, OtherViewport.GetSizeXY().Y, EWindowMode::Windowed, PF_Unknown );

		// Invalidate, then redraw immediately so the user isn't left looking at an empty black viewport
		// as they continue to resize the window.
		Invalidate();
	}

	// Play world viewports should transfer active stats so it doesn't appear like a separate viewport
	SwapStatCommands(OtherViewport);
}


void FSceneViewport::SwapStatCommands( const FSceneViewport& OtherViewport )
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FViewportClient* const ClientA = ViewportClient;
	FViewportClient* const ClientB = OtherViewport.ViewportClient;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	check(ClientA && ClientB);
	// Only swap if both viewports have stats
	const TArray<FString>* StatsA = ClientA->GetEnabledStats();
	const TArray<FString>* StatsB = ClientB->GetEnabledStats();
	if (StatsA && StatsB)
	{
		const TArray<FString> StatsCopy = *StatsA;
		ClientA->SetEnabledStats(*StatsB);
		ClientB->SetEnabledStats(StatsCopy);
	}
}

void FSceneViewport::InitRHI(FRHICommandListBase& RHICmdList)
{
	if(bRequiresHitProxyStorage)
	{
		// Initialize the hit proxy map.
		HitProxyMap.Init(RHICmdList, SizeX,SizeY);
	}
	RTTSize = FIntPoint(0, 0);

	uint32 TexSizeX = SizeX, TexSizeY = SizeY;

	static const auto CVarDefaultBackBufferPixelFormat = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.DefaultBackBufferPixelFormat"));
	SceneTargetFormat = EDefaultBackBufferPixelFormat::Convert2PixelFormat(EDefaultBackBufferPixelFormat::FromInt(CVarDefaultBackBufferPixelFormat->GetValueOnRenderThread()));

	if (bHDRViewport)
	{
		SceneTargetFormat = GRHIHDRDisplayOutputFormat;
	}

	if (UseSeparateRenderTarget())
	{
		int32 NewNumBufferedFrames = 1;
		TArray<FTextureRHIRef> ExternalRenderTargets;
		TArray<FTextureRHIRef> ExternalResourceTextures;
		
		// @todo vreditor switch: This code needs to be called when switching between stereo/non when going immersive.  Seems to always work out that way anyway though? (Probably due to resize)
		bool bExternallyAllocatedRenderTargets = false;
		if (IStereoRenderTargetManager* StereoRenderTargetManager = RetrieveStereoRenderTargetManager(IsStereoRenderingAllowed()))
		{
			StereoRenderTargetManager->CalculateRenderTargetSize(*this, TexSizeX, TexSizeY);
			bExternallyAllocatedRenderTargets = StereoRenderTargetManager->AllocateRenderTargetTextures(RHICmdList, TexSizeX, TexSizeY, SceneTargetFormat, 1, TexCreate_None, TexCreate_RenderTargetable, ExternalRenderTargets, ExternalResourceTextures);
			if (bExternallyAllocatedRenderTargets)
			{
				check(ExternalRenderTargets.Num() == ExternalResourceTextures.Num());
				check(ExternalRenderTargets.Num() > 0);
				NewNumBufferedFrames = ExternalRenderTargets.Num();
			}
		}

		// clear existing entries
		for (FFrameData& FrameData : BufferedFrames)
		{
			if (!FrameData.SlateHandle)
			{
				FrameData.SlateHandle = new FSlateRenderTargetRHI(nullptr, 0, 0);
			}
			FrameData.RenderTarget = nullptr;
			FrameData.ResourceTexture = nullptr;
		}

		if (BufferedFrames.Num() < NewNumBufferedFrames)
		{
			// Add sufficient entries for buffering.
			for (int32 Index = BufferedFrames.Num(); Index < NewNumBufferedFrames; Index++)
			{
				BufferedFrames.Emplace(new FSlateRenderTargetRHI(nullptr, 0, 0));
			}
		}
		else if (BufferedFrames.Num() > NewNumBufferedFrames)
		{
			BufferedFrames.SetNum(NewNumBufferedFrames);
		}

		for (int32 Index = 0; Index < NewNumBufferedFrames; Index++)
		{
			FTextureRHIRef RenderTarget;
			FTextureRHIRef ResourceTexture;

			if (bExternallyAllocatedRenderTargets)
			{
				RenderTarget = MoveTemp(ExternalRenderTargets[Index]);
				ResourceTexture = MoveTemp(ExternalResourceTextures[Index]);
			}
			else
			{
				const FRHITextureCreateDesc Desc =
					FRHITextureCreateDesc::Create2D(TEXT("BufferedRT"))
					.SetExtent(TexSizeX, TexSizeY)
					.SetFormat(SceneTargetFormat)
					.SetFlags(ETextureCreateFlags::RenderTargetable | ETextureCreateFlags::ShaderResource)
					.SetInitialState(ERHIAccess::SRVMask);

				RenderTarget = ResourceTexture = RHICmdList.CreateTexture(Desc);
			}

			BufferedFrames[Index].RenderTarget = RenderTarget;
			BufferedFrames[Index].ResourceTexture = ResourceTexture;

			if (BufferedFrames[Index].SlateHandle)
			{
				BufferedFrames[Index].SlateHandle->SetRHIRef(ResourceTexture, TexSizeX, TexSizeY);
			}
		}

		// clear out any extra entries we have hanging around
		for (int32 Index = NewNumBufferedFrames; Index < BufferedFrames.Num(); Index++)
		{
			if (BufferedFrames[Index].SlateHandle)
			{
				BufferedFrames[Index].SlateHandle->SetRHIRef(nullptr, 0, 0);
			}
			BufferedFrames[Index].RenderTarget = nullptr;
			BufferedFrames[Index].ResourceTexture = nullptr;
		}

		BufferIndex_GT = 0;
		BufferIndex_RT = 0;
		NextBufferIndex_GT = (BufferIndex_GT + 1) % NewNumBufferedFrames;
		NumBufferedFrames = NewNumBufferedFrames;

		RenderTargetTextureRHI = BufferedFrames[BufferIndex_GT].ResourceTexture;
	}
	else
	{
		if (BufferedFrames.Num() == 0)
		{
			BufferedFrames.Emplace();
		}

		RenderTargetTextureRHI = nullptr;
		BufferIndex_GT = 0;
		BufferIndex_RT = 0;
		NextBufferIndex_GT = 0;
		NumBufferedFrames = 0;
	}
}

void FSceneViewport::ReleaseRHI()
{
	FViewport::ReleaseRHI();

	for (FFrameData& FrameData : BufferedFrames)
	{
		if (FrameData.SlateHandle)
		{
			FrameData.SlateHandle->ReleaseRHI();
		}
	}
	if (RenderThreadSlateTexture)
	{
		RenderThreadSlateTexture->ReleaseRHI();
	}
}

void FSceneViewport::SetPreCaptureMousePosFromSlateCursor()
{
	PreCaptureCursorPos = FSlateApplication::Get().GetCursorPos().IntPoint();
}
