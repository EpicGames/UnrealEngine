// Copyright Epic Games, Inc. All Rights Reserved.

#include "Slate/SlateBrushAsset.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SlateBrushAsset)

USlateBrushAsset::USlateBrushAsset( const FObjectInitializer& ObjectInitializer )
	: Super(ObjectInitializer)
{
	
}

#if WITH_EDITORONLY_DATA
void USlateBrushAsset::PostLoad()
{
	Super::PostLoad();

	if ( Brush.Tint_DEPRECATED != FLinearColor::White )
	{
		Brush.TintColor = FSlateColor( Brush.Tint_DEPRECATED );
	}
}
#endif

