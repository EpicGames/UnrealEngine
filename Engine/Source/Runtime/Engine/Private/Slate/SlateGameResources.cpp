// Copyright Epic Games, Inc. All Rights Reserved.

#include "Slate/SlateGameResources.h"
#include "AssetRegistry/AssetData.h"
#include "Misc/Paths.h"
#include "Modules/ModuleManager.h"
#include "Styling/SlateWidgetStyle.h"
#include "Styling/SlateWidgetStyleAsset.h"
#include "EngineUtils.h"
#include "Slate/SlateBrushAsset.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "Logging/MessageLog.h"
#include "Curves/CurveVector.h"
#include "Curves/CurveLinearColor.h"
#include "Styling/SlateWidgetStyleContainerBase.h"

TSharedRef<FSlateGameResources> FSlateGameResources::New( const FName& InStyleSetName, const FString& ScopeToDirectory, const FString& InBasePath )
{
	const TSharedRef<FSlateGameResources> NewStyle = MakeShareable(new FSlateGameResources(InStyleSetName));
	NewStyle->Initialize( ScopeToDirectory, InBasePath );
	return NewStyle;
}

FSlateGameResources::FSlateGameResources( const FName& InStyleSetName ) 
	: FSlateStyleSet( InStyleSetName )
	, UIResources()
	, BasePath()
	, HasBeenInitialized( false )
{
}

FSlateGameResources::~FSlateGameResources()
{
	if ( GIsEditor && FModuleManager::Get().IsModuleLoaded( TEXT("AssetRegistry") ) )
	{
		FAssetRegistryModule& AssetRegistryModule = FModuleManager::GetModuleChecked<FAssetRegistryModule>( TEXT("AssetRegistry") );
		IAssetRegistry* AssetRegistry = AssetRegistryModule.TryGet();
		if (AssetRegistry)
		{
			AssetRegistry->OnAssetAdded().RemoveAll(this);
			AssetRegistry->OnAssetRemoved().RemoveAll(this);
		}
	}
}

void FSlateGameResources::SetContentRoot( const FString& InContentRootDir )
{
	checkf( ContentRootDir.IsEmpty(), TEXT("You can't change the root directory after setting it") );
	ContentRootDir = InContentRootDir;
}

const FSlateBrush* FSlateGameResources::GetBrush( const FName PropertyName, const ANSICHAR* Specifier, const ISlateStyle* RequestingStyle ) const
{
	ensureMsgf(Specifier == NULL, TEXT("Attempting to look up resource (%s, %s). \n Specifiers not supported by Slate Resource Sets loaded from content browser."), *PropertyName.ToString(), ANSI_TO_TCHAR(Specifier));
	FName CleanName = GetCleanName(PropertyName);
	auto* Resource = UIResources.Find(CleanName);
	if(Resource)
	{
		const USlateBrushAsset* BrushAsset = Cast<USlateBrushAsset>(*Resource);
		ensureMsgf(BrushAsset, TEXT("Could not find resource '%s'"), *CleanName.ToString());
		return BrushAsset ? &BrushAsset->Brush : GetDefaultBrush();
	}
	return FSlateStyleSet::GetBrush(PropertyName, Specifier, RequestingStyle);
}

const FSlateBrush* FSlateGameResources::GetOptionalBrush(const FName PropertyName, const ANSICHAR* Specifier, const FSlateBrush* const InDefaultBrush) const
{
	ensureMsgf(Specifier == NULL, TEXT("Attempting to look up resource (%s, %s). \n Specifiers not supported by Slate Resource Sets loaded from content browser."), *PropertyName.ToString(), ANSI_TO_TCHAR(Specifier));
	FName CleanName = GetCleanName(PropertyName);
	auto* Resource = UIResources.Find(CleanName);
	if(Resource)
	{
		const USlateBrushAsset* BrushAsset = Cast<USlateBrushAsset>(*Resource);
		ensureMsgf(BrushAsset, TEXT("Could not find resource '%s'"), *CleanName.ToString());
		return BrushAsset ? &BrushAsset->Brush : InDefaultBrush;
	}
	return FSlateStyleSet::GetOptionalBrush(PropertyName, Specifier, InDefaultBrush);
}

UCurveFloat* FSlateGameResources::GetCurveFloat( const FName AssetName ) const
{
	FName CleanName = GetCleanName(AssetName);
	auto* Resource = UIResources.Find(CleanName);
	UCurveFloat* Curve = Resource ? Cast<UCurveFloat>(*Resource) : NULL;
	ensureMsgf(Curve, TEXT("Could not find resource '%s'"), *CleanName.ToString());
	return Curve;
}

UCurveVector* FSlateGameResources::GetCurveVector( const FName AssetName ) const
{
	FName CleanName = GetCleanName(AssetName);
	auto* Resource = UIResources.Find(CleanName);
	UCurveVector* Curve = Resource ? Cast<UCurveVector>(*Resource) : NULL;
	ensureMsgf(Curve, TEXT("Could not find resource '%s'"), *CleanName.ToString());
	return Curve;
}

UCurveLinearColor* FSlateGameResources::GetCurveLinearColor( const FName AssetName ) const
{
	FName CleanName = GetCleanName(AssetName);
	auto* Resource = UIResources.Find(CleanName);
	UCurveLinearColor* Curve = Resource ? Cast<UCurveLinearColor>(*Resource) : NULL;
	ensureMsgf(Curve, TEXT("Could not find resource '%s'"), *CleanName.ToString());
	return Curve;
}

void FSlateGameResources::GetResources( TArray< const FSlateBrush* >& OutResources ) const
{
	FSlateStyleSet::GetResources( OutResources );

	for( auto PairIter = UIResources.CreateConstIterator(); PairIter; ++PairIter )
	{
		const USlateWidgetStyleAsset* Style = Cast<USlateWidgetStyleAsset>(PairIter->Value);
		const USlateBrushAsset* SlateBrushAsset = Cast<USlateBrushAsset>(PairIter->Value);
		if ( Style && Style->CustomStyle )
		{
			const FSlateWidgetStyle* Definition = Style->CustomStyle->GetStyle();
			if ( Definition != NULL )
			{
				Definition->GetResources( OutResources );
			}
		}
		else if ( SlateBrushAsset )
		{
			OutResources.AddUnique(&SlateBrushAsset->Brush);
		}
	}
}

const FSlateWidgetStyle* FSlateGameResources::GetWidgetStyleInternal(const FName DesiredTypeName, const FName StyleName, const FSlateWidgetStyle* DefaultStyle, bool bWarnIfNotFound) const
{
	auto* UIResourcePtr = UIResources.Find( StyleName );
	USlateWidgetStyleAsset* StyleAsset = UIResourcePtr ? Cast<USlateWidgetStyleAsset>(*UIResourcePtr) : NULL;

	if ( StyleAsset == NULL )
	{
		return FSlateStyleSet::GetWidgetStyleInternal( DesiredTypeName, StyleName, DefaultStyle, bWarnIfNotFound);
	}

	const FSlateWidgetStyle* Style = StyleAsset->GetStyleChecked( DesiredTypeName );

	if ( Style == NULL && GIsEditor)
	{
		TSharedRef< FTokenizedMessage > Message = FTokenizedMessage::Create( EMessageSeverity::Error, FText::Format( NSLOCTEXT("SlateStyleSet", "WrongWidgetStyleType", "The Slate Widget Style '{0}' is not of the desired type. Desired: '{1}', Actual: '{2}'"), FText::FromName( StyleName ), FText::FromName( DesiredTypeName ), FText::FromName( StyleAsset->CustomStyle->GetStyle()->GetTypeName() ) ) );
		Message->AddToken( FAssetNameToken::Create( StyleAsset->GetPathName(), FText::FromString( StyleAsset->GetName() ) ) );
	
		Log(Message);
	}

	return Style;
}

void FSlateGameResources::Log( ISlateStyle::EStyleMessageSeverity Severity, const FText& Message ) const
{
	if (GIsEditor)
	{
		const EMessageSeverity::Type EngineMessageSeverity = [Severity]()
		{
			switch (Severity)
			{
			default:
			case ISlateStyle::EStyleMessageSeverity::Error:					return EMessageSeverity::Error;
			case ISlateStyle::EStyleMessageSeverity::PerformanceWarning:	return EMessageSeverity::PerformanceWarning;
			case ISlateStyle::EStyleMessageSeverity::Warning:				return EMessageSeverity::Warning;
			case ISlateStyle::EStyleMessageSeverity::Info:					return EMessageSeverity::Info;
			}
		}();

		FMessageLog SlateStyleLog("SlateStyleLog");
		SlateStyleLog.AddMessage(FTokenizedMessage::Create(EngineMessageSeverity, Message));

		if (EngineMessageSeverity <= EMessageSeverity::Warning)
		{
			SlateStyleLog.Open();
		}
	}
}

void FSlateGameResources::Log( const TSharedRef< class FTokenizedMessage >& Message ) const
{
	if (GIsEditor)
	{
		FMessageLog SlateStyleLog("SlateStyleLog");
		SlateStyleLog.AddMessage(Message);

		if (Message->GetSeverity() <= EMessageSeverity::Warning)
		{
			SlateStyleLog.Open();
		}
	}
}

void FSlateGameResources::Initialize( const FString& ScopeToDirectory, const FString& InBasePath )
{
	UIResources.Empty();
	SetContentRoot( ScopeToDirectory );
	BasePath = InBasePath;

	TArray<UObject*> LoadedObjects;
	if (EngineUtils::FindOrLoadAssetsByPath(ContentRootDir, LoadedObjects, EngineUtils::ATL_Regular))
	{
		for( auto ObjectIter = LoadedObjects.CreateConstIterator(); ObjectIter; ++ObjectIter )
		{
			AddAssetToCache( *ObjectIter, true );
		}
	}

	if ( !HasBeenInitialized && GIsEditor )
	{
		FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>( TEXT("AssetRegistry") );
		IAssetRegistry& AssetRegistry = AssetRegistryModule.Get();

		AssetRegistry.OnAssetAdded().AddRaw( this, &FSlateGameResources::AddAsset );
		AssetRegistry.OnAssetRemoved().AddRaw( this, &FSlateGameResources::RemoveAsset );
	}

	HasBeenInitialized = true;
}

void FSlateGameResources::AddAsset(const FAssetData& InAddedAssetData)
{
	if ( ShouldCache( InAddedAssetData ) ) 
	{
		AddAssetToCache( InAddedAssetData.GetAsset(), false );
	}
}

void FSlateGameResources::RemoveAsset(const FAssetData& InRemovedAssetData)
{
	RemoveAssetFromCache( InRemovedAssetData );
}

bool FSlateGameResources::ShouldCache( const FAssetData& InAssetData )
{
	return InAssetData.PackageName.ToString().StartsWith( ContentRootDir, ESearchCase::CaseSensitive )
		&& InAssetData.AssetClassPath == USlateWidgetStyleAsset::StaticClass()->GetClassPathName();
}

void FSlateGameResources::AddAssetToCache( UObject* InStyleObject, bool bEnsureUniqueness )
{
	const USlateWidgetStyleAsset* StyleObject = Cast<USlateWidgetStyleAsset>(InStyleObject);
	const USlateBrushAsset* SlateBrushAsset = Cast<USlateBrushAsset>(InStyleObject);
	const UCurveBase* CurveAsset = Cast<UCurveBase>(InStyleObject);
	const bool bSupportedAssetType = StyleObject != NULL || SlateBrushAsset != NULL || CurveAsset != NULL;
	if ( bSupportedAssetType )
	{
		const FName StyleName = GenerateMapName( InStyleObject );
		TObjectPtr<UObject>* ExistingAsset = NULL;
		if ( bEnsureUniqueness )
		{
			ExistingAsset = UIResources.Find( StyleName );
		}

		if ( ExistingAsset != NULL )
		{
			Log(EStyleMessageSeverity::Error, FText::Format( NSLOCTEXT("SlateWidgetStyleSet", "LoadingError", "Encountered multiple Slate Widget Styles with the same name. Name: '{0}', First Asset: '{1}',  Second Asset: '{2}'."),
				FText::FromName( StyleName ),
				FText::FromString( InStyleObject->GetPathName() ),
				FText::FromString( InStyleObject->GetPathName() ) ) );
		}
		else
		{
			UIResources.Add( StyleName, InStyleObject );
		}
	}
}

void FSlateGameResources::RemoveAssetFromCache( const FAssetData& AssetData )
{
	if ( ShouldCache( AssetData ) )
	{
		UIResources.Remove( GenerateMapName( AssetData ) );
	}
}

FName FSlateGameResources::GenerateMapName( const FAssetData& AssetData )
{
	const FString PackagePath = AssetData.PackagePath.ToString();
	FString MapName = PackagePath.Right( PackagePath.Len() - BasePath.Len() );

	if ( MapName.IsEmpty() )
	{
		MapName = AssetData.AssetName.ToString();
	}
	else
	{
		MapName += TEXT("/");
		MapName += AssetData.AssetName.ToString();
	}

	return FName( *MapName );
}

FName FSlateGameResources::GenerateMapName( UObject* StyleObject )
{
	return GenerateMapName( FAssetData( StyleObject ) );
}

FName FSlateGameResources::GetCleanName(const FName& AssetName) const
{
	static const FString SplitOn = "'";
	FString RetValue = AssetName.ToString();
	if (!FPaths::GetBaseFilename(RetValue, false).Split(SplitOn, NULL, &RetValue))
	{
		return AssetName;
	}
	return FName(*RetValue);
}

void FSlateGameResources::AddReferencedObjects( FReferenceCollector& Collector )
{
	Collector.AddReferencedObjects( UIResources );
}

FString FSlateGameResources::GetReferencerName() const
{
	return TEXT("FSlateGameResources");
}
