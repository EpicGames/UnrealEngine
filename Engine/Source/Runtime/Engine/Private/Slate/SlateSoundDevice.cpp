// Copyright Epic Games, Inc. All Rights Reserved.

#include "Slate/SlateSoundDevice.h"
#include "Sound/SlateSound.h"
#include "AudioDevice.h"
#include "Audio/AudioTraceUtil.h"
#include "Engine/Engine.h"

DEFINE_LOG_CATEGORY_STATIC(LogSlateSoundDevice, Log, All);

void FSlateSoundDevice::PlaySound(const FSlateSound& Sound, int32 UserIndex) const
{
	if( GEngine && Sound.GetResourceObject() != nullptr )
	{
		FAudioDeviceHandle AudioDevice = GEngine->GetActiveAudioDevice();
		if(AudioDevice)
		{
			UObject* const Object = Sound.GetResourceObject();
			USoundBase* const SoundResource = Cast<USoundBase>(Object);
			if(SoundResource)
			{
				FActiveSound NewActiveSound;
				NewActiveSound.SetSound(SoundResource);
				NewActiveSound.bIsUISound = true;
				NewActiveSound.UserIndex = UserIndex;
				NewActiveSound.Priority = SoundResource->Priority;

#if UE_AUDIO_PROFILERTRACE_ENABLED
				const FString* EventLogID = &Audio::Trace::EventLog::ID::PlayRequestSlateSound;
#else
				const FString* EventLogID = nullptr;
#endif // UE_AUDIO_PROFILERTRACE_ENABLED

				AudioDevice->AddNewActiveSound(NewActiveSound, nullptr, EventLogID);
			}
			else if(Object)
			{
				// An Object but no SoundResource means that the FSlateSound is holding an invalid object; report that as an error
				UE_LOGF(LogSlateSoundDevice, Error, "A sound contains a non-sound resource '%ls'", *Object->GetName());
			}
		}
	}
}

float FSlateSoundDevice::GetSoundDuration(const FSlateSound& Sound) const
{
	USoundBase* const SoundResource = Cast<USoundBase>(Sound.GetResourceObject());
	return (SoundResource) ? SoundResource->Duration : 0.0f;
}
