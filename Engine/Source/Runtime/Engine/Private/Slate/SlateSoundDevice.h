// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Sound/ISlateSoundDevice.h"

struct FSlateSound;

class FSlateSoundDevice : public ISlateSoundDevice
{
public:
	virtual ~FSlateSoundDevice()= default;

private:
	virtual void PlaySound(const FSlateSound& Sound, int32 UserIndex = 0) const override;	
	virtual float GetSoundDuration(const FSlateSound& Sound) const override;
};
