// Copyright Epic Games, Inc. All Rights Reserved.

#include "Slate/SlateTextureAtlasInterface.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SlateTextureAtlasInterface)

USlateTextureAtlasInterface::USlateTextureAtlasInterface(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

