// Copyright Epic Games, Inc. All Rights Reserved.


#include "Slate/SlateTextures.h"
#include "Engine/Texture.h"
#include "ProfilingDebugging/LoadTimeTracker.h"
#include "RenderingThread.h"
#include "RHIUtilities.h"

FSlateTexture2DRHIRef::FSlateTexture2DRHIRef( FTextureRHIRef InRef, uint32 InWidth, uint32 InHeight )
	: TSlateTexture( InRef )
	, Width( InWidth )
	, Height( InHeight )
	, TexCreateFlags( TexCreate_None )
	, PixelFormat( PF_Unknown )
	, bCreateEmptyTexture( false )
{

}

FSlateTexture2DRHIRef::FSlateTexture2DRHIRef( uint32 InWidth, uint32 InHeight, EPixelFormat InPixelFormat, TSharedPtr<FSlateTextureData, ESPMode::ThreadSafe> InTextureData, ETextureCreateFlags InTexCreateFlags, bool bInCreateEmptyTexture)
	: Width( InWidth )
	, Height( InHeight )
	, TexCreateFlags( InTexCreateFlags )
	, TextureData( InTextureData )
	, PixelFormat( InPixelFormat )
	, bCreateEmptyTexture( bInCreateEmptyTexture )
{

}

FSlateTexture2DRHIRef::~FSlateTexture2DRHIRef()
{

}

void FSlateTexture2DRHIRef::Cleanup()
{
	BeginReleaseResource(this);
	BeginCleanup(this);
}

const FName NAME_FSlateTexture2DRHIRef("FSlateTexture2DRHIRef");

void FSlateTexture2DRHIRef::InitRHI(FRHICommandListBase& RHICmdList)
{
	SCOPED_LOADTIMER(FSlateTexture2DRHIRef_InitDynamicRHI);

	check( IsInRenderingThread() );

	if( Width > 0 && Height > 0 )
	{
		const bool bValid = TextureData.IsValid();
		const bool bHasUploadData = TextureData.IsValid() && TextureData->GetRawBytes().Num() > 0;

		if (bValid || bHasUploadData || bCreateEmptyTexture)
		{
			check(!IsValidRef(ShaderResource));

			FRHITextureCreateDesc CreateDesc =
				FRHITextureCreateDesc::Create2D(TEXT("FSlateTexture2DRHIRef"), Width, Height, PixelFormat)
				.SetFlags(TexCreateFlags)
				.SetClassName(NAME_FSlateTexture2DRHIRef);

			if (bHasUploadData)
			{
				CreateDesc.SetInitActionInitializer();
			}

			FRHITextureInitializer Initializer = RHICmdList.CreateTextureInitializer(CreateDesc);

			if (bHasUploadData)
			{
				check(Width == TextureData->GetWidth());
				check(Height == TextureData->GetHeight());

				const FRHITextureSubresourceInitializer Subresource = Initializer.GetTexture2DSubresource(0);

				const uint8* SourceTextureData = TextureData->GetRawBytes().GetData();

				const uint32 BlocksX = GPixelFormats[PixelFormat].GetBlockCountForWidth(Width);
				const uint32 BlocksY = GPixelFormats[PixelFormat].GetBlockCountForHeight(Height);
				const uint32 SourceDataStride = BlocksX * GPixelFormats[PixelFormat].BlockBytes;

				checkf((uint32)TextureData->GetRawBytes().Num() >= SourceDataStride * BlocksY, TEXT("Not enough bytes in source TextureData to complete copy operation"));

				if (Subresource.Stride == SourceDataStride)
				{
					FMemory::Memcpy(Subresource.Data, SourceTextureData, Subresource.Stride * BlocksY);
				}
				else
				{
					uint8* WritePointer = reinterpret_cast<uint8*>(Subresource.Data);

					checkf(SourceDataStride < Subresource.Stride, TEXT("Texture stride of %llu is smaller than source data stride of %u, PixelFormat=%s (%d)"), Subresource.Stride, SourceDataStride, GPixelFormats[PixelFormat].Name, (int32)PixelFormat);
					for (uint32 i = 0; i < BlocksY; i++)
					{
						FMemory::Memcpy(WritePointer, SourceTextureData, SourceDataStride);
						WritePointer += Subresource.Stride;
						SourceTextureData += SourceDataStride;
					}
				}
				TextureData->Empty();
			}

			ShaderResource = Initializer.Finalize();
			check(IsValidRef(ShaderResource));

			INC_MEMORY_STAT_BY(STAT_SlateTextureGPUMemory, Width * Height * GPixelFormats[PixelFormat].BlockBytes);
		}
	}
}

void FSlateTexture2DRHIRef::ReleaseRHI()
{
	check( IsInRenderingThread() );

	if( IsValidRef(ShaderResource) )
	{
		DEC_MEMORY_STAT_BY(STAT_SlateTextureGPUMemory, Width*Height*GPixelFormats[PixelFormat].BlockBytes);
	}

	ShaderResource.SafeRelease();

}

void FSlateTexture2DRHIRef::Resize( uint32 InWidth, uint32 InHeight )
{
	Width = InWidth;
	Height = InHeight;
	UpdateRHI(FRHICommandListImmediate::Get());
}

void FSlateTexture2DRHIRef::SetRHIRef( FTextureRHIRef InRHIRef, uint32 InWidth, uint32 InHeight )
{
	check( IsInRenderingThread() );
	ShaderResource = InRHIRef;
	Width = InWidth;
	Height = InHeight;
}

void FSlateTexture2DRHIRef::SetTextureData( FSlateTextureDataPtr NewTextureData )
{
	check( IsInRenderingThread() );

	Width = NewTextureData->GetWidth();
	Height = NewTextureData->GetHeight();
	TextureData = NewTextureData;

	// we will memcpy the TextureData into the RHI texture without checks later, so they must match exactly
	//	typically BytesPerPixel == 4 and PixelFormat == BGRA8 , but other things are possible here for viewports & movies and such.
	check( NewTextureData->GetBytesPerPixel() == GPixelFormats[PixelFormat].BlockBytes );
}

void FSlateTexture2DRHIRef::SetTextureData( FSlateTextureDataPtr NewTextureData, EPixelFormat InPixelFormat, ETextureCreateFlags InTexCreateFlags )
{
	check( IsInRenderingThread() );
	
	PixelFormat = InPixelFormat;
	TexCreateFlags = InTexCreateFlags;

	SetTextureData( NewTextureData );
}


void FSlateTexture2DRHIRef::ClearTextureData()
{
	check( IsInRenderingThread() );
	TextureData.Reset();
}

void FSlateTexture2DRHIRef::ResizeTexture(uint32 InWidth, uint32 InHeight)
{
	if (GetWidth() != InWidth || GetHeight() != InHeight)
	{
		if (IsInRenderingThread())
		{
			Resize(InWidth, InHeight);
		}
		else
		{
			FIntPoint Dimensions(InWidth, InHeight);
			FSlateTexture2DRHIRef* TextureRHIRef = this;
			ENQUEUE_RENDER_COMMAND(ResizeSlateTexture)(
				[TextureRHIRef, Dimensions](FRHICommandListImmediate& RHICmdList)
				{
					TextureRHIRef->Resize(Dimensions.X, Dimensions.Y);
				});
		}
	}
}

void FSlateTexture2DRHIRef::SetTextureData(FRHICommandListImmediate& RHICmdList, const TArray<uint8>& Bytes)
{
	uint32 DstStride = 0;
	uint8* DstData = (uint8*)RHICmdList.LockTexture2D(GetTypedResource(), 0, RLM_WriteOnly, DstStride, false);

	const uint32 NumBlocksX = (Width  + GPixelFormats[PixelFormat].BlockSizeX - 1) / GPixelFormats[PixelFormat].BlockSizeX;
	const uint32 NumBlocksY = (Height + GPixelFormats[PixelFormat].BlockSizeY - 1) / GPixelFormats[PixelFormat].BlockSizeY;
	const uint32 SrcStride = NumBlocksX * GPixelFormats[PixelFormat].BlockBytes;
	ensure(SrcStride * NumBlocksY == Bytes.Num());

	if (SrcStride == DstStride)
	{
		FMemory::Memcpy(DstData, Bytes.GetData(), Bytes.Num());
	}
	else
	{
		const uint8* SrcData = Bytes.GetData();
		for (uint32 Row = 0; Row < NumBlocksY; ++Row)
		{
			FMemory::Memcpy(DstData, SrcData, SrcStride);
			DstData += DstStride;
			SrcData += SrcStride;
		}
	}

	RHICmdList.UnlockTexture2D(GetTypedResource(), 0, false);
}

void FSlateTexture2DRHIRef::UpdateTexture(const TArray<uint8>& Bytes)
{
	if (IsInRenderingThread())
	{
		SetTextureData(FRHICommandListImmediate::Get(), Bytes);
	}
	else
	{
		FSlateTexture2DRHIRef* TextureRHIRef = this;
		ENQUEUE_RENDER_COMMAND(UpdateSlateTexture)(
			[TextureRHIRef, Bytes](FRHICommandListImmediate& RHICmdList)
			{
				TextureRHIRef->SetTextureData(RHICmdList, Bytes);
			});
	}
}

void FSlateTexture2DRHIRef::UpdateTextureThreadSafe(const TArray<uint8>& Bytes)
{
	if(IsInGameThread())
	{
		// Make bulk data for updating the texture memory later
		uint32 BytesPerPixel = GPixelFormats[PixelFormat].BlockBytes;
		check( Width*Height*BytesPerPixel == Bytes.Num() );
		FSlateTextureData* BulkData = new FSlateTextureData( Width,Height,BytesPerPixel, Bytes );

		// Update the texture RHI
		FSlateTexture2DRHIRef* ThisTexture = this;
		ENQUEUE_RENDER_COMMAND(FSlateTexture2DRHIRef_UpdateTextureThreadSafe)(
			[ThisTexture, BulkData](FRHICommandListImmediate& RHICmdList)
			{
				ThisTexture->UpdateTexture( BulkData->GetRawBytes() );
				delete BulkData;
			});
	}
}

void FSlateTexture2DRHIRef::UpdateTextureThreadSafeRaw(uint32 InWidth, uint32 InHeight, const void* Buffer, const FIntRect& Dirty)
{
	if (IsInGameThread())
	{
		// No cheap way to avoid having to copy the Buffer, as we cannot guarantee it will not be touched before the rendering thread is done with it.
		FSlateTextureData* BulkData = new FSlateTextureData( (uint8*)Buffer, InWidth, InHeight, 4 );
		UpdateTextureThreadSafeWithTextureData(BulkData);
	}
}

void FSlateTexture2DRHIRef::UpdateTextureThreadSafeWithTextureData(FSlateTextureData* BulkData)
{
	check(IsInGameThread());
	
	// Update the texture RHI
	// BEWARE this is enqueued, but is not holding a ref on "BulkData" ; should probably use a TSharedPtr<FSlateTextureData>

	// we will memcpy the TextureData into the RHI texture without checks later, so they must match exactly
	//	typically BytesPerPixel == 4 and PixelFormat == BGRA8 , but other things are possible here for viewports & movies and such.
	check( BulkData->GetBytesPerPixel() == GPixelFormats[PixelFormat].BlockBytes );

	FSlateTexture2DRHIRef* ThisTexture = this;
	ENQUEUE_RENDER_COMMAND(FSlateTexture2DRHIRef_UpdateTextureThreadSafeWithTextureData)(
		[ThisTexture, BulkData](FRHICommandListImmediate& RHICmdList)
		{
			if (ThisTexture->GetWidth() != BulkData->GetWidth() || ThisTexture->GetHeight() != BulkData->GetHeight())
			{
				ThisTexture->Resize(BulkData->GetWidth(), BulkData->GetHeight());
			}
			ThisTexture->UpdateTexture(BulkData->GetRawBytes());
			delete BulkData;
		});
}

void FSlateRenderTargetRHI::SetRHIRef( FTextureRHIRef InRenderTargetTexture, uint32 InWidth, uint32 InHeight )
{
	check( IsInRenderingThread() );
	ShaderResource = InRenderTargetTexture;
	Width = InWidth;
	Height = InHeight;
}





FSlateTextureRenderTarget2DResource::FSlateTextureRenderTarget2DResource(const FLinearColor& InClearColor, int32 InTargetSizeX, int32 InTargetSizeY, uint8 InFormat, ESamplerFilter InFilter, TextureAddress InAddressX, TextureAddress InAddressY, float InTargetGamma)
	:	ClearColor(InClearColor)
	,   TargetSizeX(InTargetSizeX)
	,	TargetSizeY(InTargetSizeY)
	,	Format(InFormat)
	,	Filter(InFilter)
	,	AddressX(InAddressX)
	,	AddressY(InAddressY)
	,	TargetGamma(InTargetGamma)
{
}

void FSlateTextureRenderTarget2DResource::SetSize(int32 InSizeX,int32 InSizeY)
{
	if (InSizeX != TargetSizeX || InSizeY != TargetSizeY)
	{
		TargetSizeX = InSizeX;
		TargetSizeY = InSizeY;
		// reinit the resource with new TargetSizeX,TargetSizeY
		UpdateRHI(FRHICommandListImmediate::Get());
	}	
}

void FSlateTextureRenderTarget2DResource::ClampSize(int32 MaxSizeX,int32 MaxSizeY)
{
	// upsize to go back to original or downsize to clamp to max
	int32 NewSizeX = FMath::Min<int32>(TargetSizeX,MaxSizeX);
	int32 NewSizeY = FMath::Min<int32>(TargetSizeY,MaxSizeY);
	if (NewSizeX != TargetSizeX || NewSizeY != TargetSizeY)
	{
		TargetSizeX = NewSizeX;
		TargetSizeY = NewSizeY;
		// reinit the resource with new TargetSizeX,TargetSizeY
		UpdateRHI(FRHICommandListImmediate::Get());
	}	
}

void FSlateTextureRenderTarget2DResource::InitRHI(FRHICommandListBase& RHICmdList)
{
	SCOPED_LOADTIMER(FSlateTextureRenderTarget2DResource_InitDynamicRHI);

	check(IsInRenderingThread());

	if( TargetSizeX > 0 && TargetSizeY > 0 )
	{
		const static FLazyName ClassName(TEXT("FSlateTextureRenderTarget2DResource"));

		// Create the RHI texture. Only one mip is used and the texture is targetable for resolve.
		const FRHITextureCreateDesc Desc =
			FRHITextureCreateDesc::Create2D(TEXT("FSlateTextureRenderTarget2DResource"))
			.SetExtent(TargetSizeX, TargetSizeY)
			.SetFormat((EPixelFormat)Format)
			.SetClearValue(FClearValueBinding(ClearColor))
			.SetFlags(ETextureCreateFlags::RenderTargetable | ETextureCreateFlags::ShaderResource)
			.SetInitialState(ERHIAccess::SRVMask)
			.SetClassName(ClassName);

		RenderTargetTextureRHI = TextureRHI = RHICmdList.CreateTexture(Desc);
	}

	// Create the sampler state RHI resource.
	FSamplerStateInitializerRHI SamplerStateInitializer
	(
		Filter,
		AddressX == TA_Wrap ? AM_Wrap : (AddressX == TA_Clamp ? AM_Clamp : AM_Mirror),
		AddressY == TA_Wrap ? AM_Wrap : (AddressY == TA_Clamp ? AM_Clamp : AM_Mirror),
		AM_Wrap
	);
	SamplerStateRHI = GetOrCreateSamplerState( SamplerStateInitializer );
}

void FSlateTextureRenderTarget2DResource::ReleaseRHI()
{
	check(IsInRenderingThread());

	// Release the FTexture RHI resources here as well
	FTexture::ReleaseRHI();

	RenderTargetTextureRHI.SafeRelease();

	// Remove from global list of deferred clears
	RemoveFromDeferredUpdateList();
}

void FSlateTextureRenderTarget2DResource::UpdateDeferredResource(FRHICommandListImmediate& RHICmdList, bool bClearRenderTarget/*=true*/)
{
	check(IsInRenderingThread());

	// Clear the target surface to green
	if (bClearRenderTarget)
	{
		RHICmdList.Transition(FRHITransitionInfo(RenderTargetTextureRHI, ERHIAccess::Unknown, ERHIAccess::RTV));
		ClearRenderTarget(RHICmdList, RenderTargetTextureRHI);
		RHICmdList.Transition(FRHITransitionInfo(RenderTargetTextureRHI, ERHIAccess::RTV, ERHIAccess::SRVGraphics));
	}
}

uint32 FSlateTextureRenderTarget2DResource::GetSizeX() const
{ 
	return TargetSizeX; 
}

uint32 FSlateTextureRenderTarget2DResource::GetSizeY() const
{ 
	return TargetSizeY; 
}

FIntPoint FSlateTextureRenderTarget2DResource::GetSizeXY() const
{ 
	return FIntPoint(TargetSizeX, TargetSizeY); 
}

float FSlateTextureRenderTarget2DResource::GetDisplayGamma() const
{
	// FSlateTextureRenderTarget2DResource doesn't have Owner
	//return Owner->GetDisplayGamma();

	if (TargetGamma > UE_KINDA_SMALL_NUMBER * 10.0f)
	{
		return TargetGamma;
	}
	if (Format == PF_FloatRGB || Format == PF_FloatRGBA )
	{
		return 1.0f;
	}
	return FTextureRenderTargetResource::GetDisplayGamma();
}
