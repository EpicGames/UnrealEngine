// Copyright Epic Games, Inc. All Rights Reserved.

#include "Sound/SoundAttenuation.h"

#include "AudioDevice.h"
#include "UObject/AnimPhysObjectVersion.h"
#include "UObject/FortniteMainBranchObjectVersion.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SoundAttenuation)

/*-----------------------------------------------------------------------------
	USoundAttenuation implementation.
-----------------------------------------------------------------------------*/

FSoundAttenuationSettings::FSoundAttenuationSettings()
	: bAttenuate(true)
	, bSpatialize(true)
	, bAttenuateWithLPF(false)
	, bEnableListenerFocus(false)
	, bEnableFocusInterpolation(false)
	, bEnableOcclusion(false)
	, bUseComplexCollisionForOcclusion(false)
	, bEnableReverbSend(true)
	, bEnablePriorityAttenuation(false)
	, bApplyNormalizationToStereoSounds(false)
	, bEnableLogFrequencyScaling(false)
	, bEnableSubmixSends(false)
	, bEnableSourceDataOverride(false)
	, bEnableSendToAudioLink(true)
	, SpatializationAlgorithm(ESoundSpatializationAlgorithm::SPATIALIZATION_Default)
	, AudioLinkSettingsOverride(nullptr)
	, BinauralRadius(0.0f)
	, AbsorptionMethod(EAirAbsorptionMethod::Linear)
	, OcclusionTraceChannel(ECC_Visibility)
	, ReverbSendMethod(EReverbSendMethod::Linear)
	, PriorityAttenuationMethod(EPriorityAttenuationMethod::Linear)
#if WITH_EDITORONLY_DATA
	, DistanceType_DEPRECATED(SOUNDDISTANCE_Normal)
	, OmniRadius_DEPRECATED(0.0f)
#endif
	, NonSpatializedRadiusStart(0.0f)
	, NonSpatializedRadiusEnd(0.0f)
	, NonSpatializedRadiusMode(ENonSpatializedRadiusSpeakerMapMode::OmniDirectional)
	, StereoSpread(200.0f)
#if WITH_EDITORONLY_DATA
	, SpatializationPluginSettings_DEPRECATED(nullptr)
	, RadiusMin_DEPRECATED(400.f)
	, RadiusMax_DEPRECATED(4000.f)
#endif
	, LPFRadiusMin(3000.f)
	, LPFRadiusMax(6000.f)
	, LPFFrequencyAtMin(20000.f)
	, LPFFrequencyAtMax(20000.f)
	, HPFFrequencyAtMin(0.0f)
	, HPFFrequencyAtMax(0.0f)
	, FocusAzimuth(30.0f)
	, NonFocusAzimuth(60.0f)
	, FocusDistanceScale(1.0f)
	, NonFocusDistanceScale(1.0f)
	, FocusPriorityScale(1.0f)
	, NonFocusPriorityScale(1.0f)
	, FocusVolumeAttenuation(1.0f)
	, NonFocusVolumeAttenuation(1.0f)
	, FocusAttackInterpSpeed(1.0f)
	, FocusReleaseInterpSpeed(1.0f)
	, OcclusionLowPassFilterFrequency(20000.f)
	, OcclusionVolumeAttenuation(1.0f)
	, OcclusionInterpolationTime(0.1f)
#if WITH_EDITORONLY_DATA
	, OcclusionPluginSettings_DEPRECATED(nullptr)
	, ReverbPluginSettings_DEPRECATED(nullptr)
#endif
	, ReverbWetLevelMin(0.3f)
	, ReverbWetLevelMax(0.95f)
	, ReverbDistanceMin(UE_REAL_TO_FLOAT(AttenuationShapeExtents.X))
	, ReverbDistanceMax(UE_REAL_TO_FLOAT(AttenuationShapeExtents.X) + FalloffDistance)
	, ManualReverbSendLevel(0.0f)
	, PriorityAttenuationMin(1.0f)
	, PriorityAttenuationMax(1.0f)
	, PriorityAttenuationDistanceMin(UE_REAL_TO_FLOAT(AttenuationShapeExtents.X))
	, PriorityAttenuationDistanceMax(UE_REAL_TO_FLOAT(AttenuationShapeExtents.X) + FalloffDistance)
	, ManualPriorityAttenuation(1.0f)
{
#if WITH_EDITOR
	if (const USoundAttenuationEditorSettings* SoundAttenuationEditorSettings = GetDefault<USoundAttenuationEditorSettings>())
	{
		bEnableReverbSend = SoundAttenuationEditorSettings->bEnableReverbSend;
		bEnableSendToAudioLink = SoundAttenuationEditorSettings->bEnableSendToAudioLink;
	}
#endif // WITH_EDITOR
}

FSoundAttenuationSettings::~FSoundAttenuationSettings() = default;

FSoundAttenuationSettings::FSoundAttenuationSettings(const FSoundAttenuationSettings&) = default;

#if WITH_EDITORONLY_DATA
void FSoundAttenuationSettings::PostSerialize(const FArchive& Ar)
{
	if (Ar.UEVer() < VER_UE4_ATTENUATION_SHAPES)
	{
		FalloffDistance = RadiusMax_DEPRECATED - RadiusMin_DEPRECATED;
		const float MaxDistance = FAudioDevice::GetMaxWorldDistance();
		switch(DistanceType_DEPRECATED)
		{
		case SOUNDDISTANCE_Normal:
			AttenuationShape = EAttenuationShape::Sphere;
			AttenuationShapeExtents = FVector(RadiusMin_DEPRECATED, 0.f, 0.f);
			break;

		case SOUNDDISTANCE_InfiniteXYPlane:
			AttenuationShape = EAttenuationShape::Box;
			AttenuationShapeExtents = FVector(MaxDistance, MaxDistance, RadiusMin_DEPRECATED);
			break;

		case SOUNDDISTANCE_InfiniteXZPlane:
			AttenuationShape = EAttenuationShape::Box;
			AttenuationShapeExtents = FVector(MaxDistance, RadiusMin_DEPRECATED, MaxDistance);
			break;

		case SOUNDDISTANCE_InfiniteYZPlane:
			AttenuationShape = EAttenuationShape::Box;
			AttenuationShapeExtents = FVector(RadiusMin_DEPRECATED, MaxDistance, MaxDistance);
			break;
		}
	}

	if (Ar.IsLoading() && Ar.CustomVer(FAnimPhysObjectVersion::GUID) < FAnimPhysObjectVersion::AllowMultipleAudioPluginSettings)
	{
		if (SpatializationPluginSettings_DEPRECATED)
		{
			PluginSettings.SpatializationPluginSettingsArray.Add(SpatializationPluginSettings_DEPRECATED);
		}

		if (OcclusionPluginSettings_DEPRECATED)
		{
			PluginSettings.OcclusionPluginSettingsArray.Add(OcclusionPluginSettings_DEPRECATED);
		}

		if (ReverbPluginSettings_DEPRECATED)
		{
			PluginSettings.ReverbPluginSettingsArray.Add(ReverbPluginSettings_DEPRECATED);
		}
	}

	if (Ar.IsLoading() && Ar.CustomVer(FFortniteMainBranchObjectVersion::GUID) < FFortniteMainBranchObjectVersion::AudioAttenuationNonSpatializedRadiusBlend)
	{
		if (OmniRadius_DEPRECATED)
		{
			NonSpatializedRadiusStart = OmniRadius_DEPRECATED;
		}
	}
}
#endif

float FSoundAttenuationSettings::GetFocusPriorityScale(const struct FGlobalFocusSettings& FocusSettings, float FocusFactor) const
{
	float Focus = FocusSettings.FocusPriorityScale * FocusPriorityScale;
	float NonFocus = FocusSettings.NonFocusPriorityScale * NonFocusPriorityScale;
	float Result = FMath::Lerp(Focus, NonFocus, FocusFactor);
	return FMath::Max(0.0f, Result);
}

float FSoundAttenuationSettings::GetFocusAttenuation(const struct FGlobalFocusSettings& FocusSettings, float FocusFactor) const
{
	float Focus = FocusSettings.FocusVolumeScale * FocusVolumeAttenuation;
	float NonFocus = FocusSettings.NonFocusVolumeScale * NonFocusVolumeAttenuation;
	float Result = FMath::Lerp(Focus, NonFocus, FocusFactor);
	return FMath::Max(0.0f, Result);
}

float FSoundAttenuationSettings::GetFocusDistanceScale(const struct FGlobalFocusSettings& FocusSettings, float FocusFactor) const
{
	float Focus = FocusSettings.FocusDistanceScale * FocusDistanceScale;
	float NonFocus = FocusSettings.NonFocusDistanceScale * NonFocusDistanceScale;
	float Result = FMath::Lerp(Focus, NonFocus, FocusFactor);
	return FMath::Max(0.0f, Result);
}

bool FSoundAttenuationSettings::operator==(const FSoundAttenuationSettings& Other) const
{
	return (   bAttenuate			    == Other.bAttenuate
			&& bSpatialize			    == Other.bSpatialize
			&& dBAttenuationAtMax	    == Other.dBAttenuationAtMax
			&& FalloffMode				== Other.FalloffMode
			&& NonSpatializedRadiusStart == Other.NonSpatializedRadiusStart
			&& NonSpatializedRadiusEnd == Other.NonSpatializedRadiusEnd
			&& NonSpatializedRadiusMode == Other.NonSpatializedRadiusMode
			&& bApplyNormalizationToStereoSounds == Other.bApplyNormalizationToStereoSounds
			&& StereoSpread				== Other.StereoSpread
			&& DistanceAlgorithm	    == Other.DistanceAlgorithm
			&& AttenuationShape		    == Other.AttenuationShape
			&& bAttenuateWithLPF		== Other.bAttenuateWithLPF
			&& LPFRadiusMin				== Other.LPFRadiusMin
			&& LPFRadiusMax				== Other.LPFRadiusMax
			&& FalloffDistance		    == Other.FalloffDistance
			&& AttenuationShapeExtents	== Other.AttenuationShapeExtents
			&& SpatializationAlgorithm == Other.SpatializationAlgorithm
			&& PluginSettings.SpatializationPluginSettingsArray == Other.PluginSettings.SpatializationPluginSettingsArray
			&& LPFFrequencyAtMax		== Other.LPFFrequencyAtMax
			&& LPFFrequencyAtMin		== Other.LPFFrequencyAtMin
			&& HPFFrequencyAtMax		== Other.HPFFrequencyAtMax
			&& HPFFrequencyAtMin		== Other.HPFFrequencyAtMin
			&& bEnableLogFrequencyScaling == Other.bEnableLogFrequencyScaling
			&& bEnableSubmixSends 		== Other.bEnableSubmixSends
			&& bEnableListenerFocus 	== Other.bEnableListenerFocus
			&& bEnableSendToAudioLink	== Other.bEnableSendToAudioLink
			&& FocusAzimuth				== Other.FocusAzimuth
			&& NonFocusAzimuth			== Other.NonFocusAzimuth
			&& FocusDistanceScale		== Other.FocusDistanceScale
			&& FocusPriorityScale		== Other.FocusPriorityScale
			&& NonFocusPriorityScale	== Other.NonFocusPriorityScale
			&& FocusVolumeAttenuation	== Other.FocusVolumeAttenuation
			&& NonFocusVolumeAttenuation == Other.NonFocusVolumeAttenuation
			&& OcclusionTraceChannel	== Other.OcclusionTraceChannel
			&& OcclusionLowPassFilterFrequency == Other.OcclusionLowPassFilterFrequency
			&& OcclusionVolumeAttenuation == Other.OcclusionVolumeAttenuation
			&& OcclusionInterpolationTime == Other.OcclusionInterpolationTime
			&& PluginSettings.OcclusionPluginSettingsArray	== Other.PluginSettings.OcclusionPluginSettingsArray
			&& bEnableReverbSend		== Other.bEnableReverbSend
			&& PluginSettings.ReverbPluginSettingsArray		== Other.PluginSettings.ReverbPluginSettingsArray
			&& PluginSettings.SourceDataOverridePluginSettingsArray == Other.PluginSettings.SourceDataOverridePluginSettingsArray
			&& AudioLinkSettingsOverride == Other.AudioLinkSettingsOverride
			&& ReverbWetLevelMin		== Other.ReverbWetLevelMin
			&& ReverbWetLevelMax		== Other.ReverbWetLevelMax
			&& ReverbDistanceMin		== Other.ReverbDistanceMin
			&& ReverbDistanceMax		== Other.ReverbDistanceMax);
}

void FSoundAttenuationSettings::CollectAttenuationShapesForVisualization(TMultiMap<EAttenuationShape::Type, AttenuationShapeDetails>& ShapeDetailsMap) const
{
	if (bAttenuate)
	{
		FBaseAttenuationSettings::CollectAttenuationShapesForVisualization(ShapeDetailsMap);
	}
}
void FSoundAttenuationSettings::AddStructReferencedObjects(FReferenceCollector& Collector)
{
	Collector.AddStableReference(&AudioLinkSettingsOverride);
}

void FSoundAttenuationPluginSettings::AddStructReferencedObjects(FReferenceCollector& Collector)
{
	Collector.AddStableReferenceArray(&SpatializationPluginSettingsArray);
	Collector.AddStableReferenceArray(&OcclusionPluginSettingsArray);
	Collector.AddStableReferenceArray(&ReverbPluginSettingsArray);
	Collector.AddStableReferenceArray(&SourceDataOverridePluginSettingsArray);
}

USoundAttenuation::USoundAttenuation(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

FAttenuationSubmixSendSettings::FAttenuationSubmixSendSettings()
{
	// These were the defaults in the previous attenuation settings.
	MinSendLevel = 0.0f;
	MaxSendLevel = 1.0f;
	MinSendDistance = 400.0f;
	MaxSendDistance = 6000.0f;
	SendLevel = 0.2f;
	SendLevelControlMethod = ESendLevelControlMethod::Linear;
}

#define LOCTEXT_NAMESPACE "AudioParameterInterface"
#define AUDIO_PARAMETER_INTERFACE_NAMESPACE "UE.Attenuation"
namespace Audio
{
	namespace AttenuationInterface
	{
		const FName Name = AUDIO_PARAMETER_INTERFACE_NAMESPACE;

		namespace Inputs
		{
			const FName Distance = AUDIO_PARAMETER_INTERFACE_MEMBER_DEFINE("Distance");
		} // namespace Inputs

		Audio::FParameterInterfacePtr GetInterface()
		{
			struct FInterface : public Audio::FParameterInterface
			{
				FInterface()
					: FParameterInterface(AttenuationInterface::Name, { 1, 0 })
				{
					Inputs =
					{
						{
							FText(),
							NSLOCTEXT("AudioGeneratorInterface_Attenuation", "DistanceDescription", "Distance between listener and sound locations in Unreal Units."),
							FName(),
							{ Inputs::Distance, 0.0f }
						}
					};
#if WITH_EDITOR
					Description = LOCTEXT("UE.Attenuation.Description", "Provides the distance between listener and sound locations in Unreal Units.");
#endif
				}
			};

			static FParameterInterfacePtr InterfacePtr;
			if (!InterfacePtr.IsValid())
			{
				InterfacePtr = MakeShared<FInterface>();
			}

			return InterfacePtr;
		}
	} // namespace AttenuationInterface
#undef AUDIO_PARAMETER_INTERFACE_NAMESPACE

#define AUDIO_PARAMETER_INTERFACE_NAMESPACE "UE.Spatialization"
	namespace SpatializationInterface
	{
		const FName Name = AUDIO_PARAMETER_INTERFACE_NAMESPACE;

		namespace Inputs
		{
			const FName Azimuth = AUDIO_PARAMETER_INTERFACE_MEMBER_DEFINE("Azimuth");
			const FName Elevation = AUDIO_PARAMETER_INTERFACE_MEMBER_DEFINE("Elevation");
		} // namespace Inputs

		Audio::FParameterInterfacePtr GetInterface()
		{
			struct FInterface : public Audio::FParameterInterface
			{
				FInterface()
					: FParameterInterface(SpatializationInterface::Name, { 1, 0 })
				{
					Inputs =
					{
						{
							FText(),
							NSLOCTEXT("Spatialization", "AzimuthDescription", "[-180, 180] Horizontal angle between listener forward and sound location in degrees."),
							FName(),
							{ Inputs::Azimuth, 0.0f }
						},
						{
							FText(),
							NSLOCTEXT("Spatialization", "ElevationDescription", "[-90, 90] Vertical angle between listener forward and sound location in degrees."),
							FName(),
							{ Inputs::Elevation, 0.0f }
						}
					};
#if WITH_EDITOR
					Description = LOCTEXT("UE.Spatialization.Description",
						"From the perspective of the listener, the horizontal angle (azimuth) and vertical angle (elevation) to the sound.\n"
						"Useful for focus-based mixing, like that of the Focus feature of Attenuation Settings.");
#endif
				}
			};

			static FParameterInterfacePtr InterfacePtr;
			if (!InterfacePtr.IsValid())
			{
				InterfacePtr = MakeShared<FInterface>();
			}

			return InterfacePtr;
		}
	} // namespace SpatializationInterface
#undef AUDIO_PARAMETER_INTERFACE_NAMESPACE

#define AUDIO_PARAMETER_INTERFACE_NAMESPACE "UE.Source.Orientation"
	namespace SourceOrientationInterface
	{
		const FName Name = AUDIO_PARAMETER_INTERFACE_NAMESPACE;

		namespace Inputs
		{
			const FName Azimuth = AUDIO_PARAMETER_INTERFACE_MEMBER_DEFINE("Azimuth");
			const FName Elevation = AUDIO_PARAMETER_INTERFACE_MEMBER_DEFINE("Elevation");
		} // namespace Inputs

		Audio::FParameterInterfacePtr GetInterface()
		{
			struct FInterface : public Audio::FParameterInterface
			{
				FInterface()
					: FParameterInterface(SourceOrientationInterface::Name, { 1, 0 })
				{
					Inputs =
					{
						{
							FText(),
							NSLOCTEXT("SourceOrientation", "AzimuthDescription", "[-180, 180] Horizontal angle between sound forward and listener location in degrees."),
							FName(),
							{ Inputs::Azimuth, 0.0f }
						},
						{
							FText(),
							NSLOCTEXT("SourceOrientation", "ElevationDescription", "[-90, 90] Vertical angle between sound forward and listener location in degrees."),
							FName(),
							{ Inputs::Elevation, 0.0f }
						}
					};
#if WITH_EDITOR
					Description = LOCTEXT("UE.Source.Orientation.Description",
						"From the perspective of the sound, provides the horizontal angle (azimuth) and vertical angle (elevation) to the listener.\n"
						"Useful for directional sources like a megaphone or speaker, where the sound is louder pointed at you and quieter pointed away.");
#endif
				}
			};

			static FParameterInterfacePtr InterfacePtr;
			if (!InterfacePtr.IsValid())
			{
				InterfacePtr = MakeShared<FInterface>();
			}

			return InterfacePtr;
		}
	} // namespace SourceOrientationInterface
#undef AUDIO_PARAMETER_INTERFACE_NAMESPACE

#define AUDIO_PARAMETER_INTERFACE_NAMESPACE "UE.Listener.Orientation"
	namespace ListenerOrientationInterface
	{
		const FName Name = AUDIO_PARAMETER_INTERFACE_NAMESPACE;

		namespace Inputs
		{
			const FName Azimuth = AUDIO_PARAMETER_INTERFACE_MEMBER_DEFINE("Azimuth");
			const FName Elevation = AUDIO_PARAMETER_INTERFACE_MEMBER_DEFINE("Elevation");
		} // namespace Inputs

		Audio::FParameterInterfacePtr GetInterface()
		{
			struct FInterface : public Audio::FParameterInterface
			{
				FInterface()
					: FParameterInterface(ListenerOrientationInterface::Name, { 1, 0 })
				{
					Inputs =
					{
						{
							FText(),
							NSLOCTEXT("ListenerOrientation", "AzimuthDescription", "[-180, 180] Horizontal viewing angle of the current listener in world, in degrees."),
							FName(),
							{ Inputs::Azimuth, 0.0f }
						},
						{
							FText(),
							NSLOCTEXT("ListenerOrientation", "ElevationDescription", "[-90, 90] Vertical viewing angle of the current listener in world, in degrees."),
							FName(),
							{ Inputs::Elevation, 0.0f }
						}
					};
#if WITH_EDITOR
					Description = LOCTEXT("UE.Listener.Orientation.Description",
						"Relative to the world, provides the listener orientation, as horizontal angle (azimuth) and vertical angle (elevation).\n"
						"Useful for world-locked quad beds for example.");
#endif
				}
			};

			static FParameterInterfacePtr InterfacePtr;
			if (!InterfacePtr.IsValid())
			{
				InterfacePtr = MakeShared<FInterface>();
			}

			return InterfacePtr;
		}
	} // namespace ListenerOrientationInterface
#undef AUDIO_PARAMETER_INTERFACE_NAMESPACE

#define AUDIO_PARAMETER_INTERFACE_NAMESPACE "UE.Source.Location"
	namespace SourceLocationInterface
	{
		const FName Name = AUDIO_PARAMETER_INTERFACE_NAMESPACE;

		namespace Inputs
		{
			const FName X = AUDIO_PARAMETER_INTERFACE_MEMBER_DEFINE("X");
			const FName Y = AUDIO_PARAMETER_INTERFACE_MEMBER_DEFINE("Y");
			const FName Z = AUDIO_PARAMETER_INTERFACE_MEMBER_DEFINE("Z");
		} // namespace Inputs

		Audio::FParameterInterfacePtr GetInterface()
		{
			struct FInterface : public Audio::FParameterInterface
			{
				FInterface()
					: FParameterInterface(SourceLocationInterface::Name, { 1, 0 })
				{
					Inputs =
					{
						{
							FText(),
							NSLOCTEXT("SourceLocation", "XDescription", "X-axis of Source's World Location."),
							FName(),
							{ Inputs::X, 0.0f }
						},
						{
							FText(),
							NSLOCTEXT("SourceLocation", "YDescription", "Y-axis of Source's World Location."),
							FName(),
							{ Inputs::Y, 0.0f }
						},
						{
							FText(),
							NSLOCTEXT("SourceLocation", "ZDescription", "Z-axis of Source's World Location."),
							FName(),
							{ Inputs::Z, 0.0f }
						}
					};
#if WITH_EDITOR
					Description = LOCTEXT("UE.Source.Location.Description",
						"Provides information on the world location of the Source.");
#endif
				}
			};

			static FParameterInterfacePtr InterfacePtr;
			if (!InterfacePtr.IsValid())
			{
				InterfacePtr = MakeShared<FInterface>();
			}

			return InterfacePtr;
		}
	} // namespace SourceLocationInterface
#undef AUDIO_PARAMETER_INTERFACE_NAMESPACE

#define AUDIO_PARAMETER_INTERFACE_NAMESPACE "UE.Listener.Location"
	namespace ListenerLocationInterface
	{
		const FName Name = AUDIO_PARAMETER_INTERFACE_NAMESPACE;

		namespace Inputs
		{
			const FName X = AUDIO_PARAMETER_INTERFACE_MEMBER_DEFINE("X");
			const FName Y = AUDIO_PARAMETER_INTERFACE_MEMBER_DEFINE("Y");
			const FName Z = AUDIO_PARAMETER_INTERFACE_MEMBER_DEFINE("Z");
		} // namespace Inputs

		Audio::FParameterInterfacePtr GetInterface()
		{
			struct FInterface : public Audio::FParameterInterface
			{
				FInterface()
					: FParameterInterface(ListenerLocationInterface::Name, { 1, 0 })
				{
					Inputs =
					{
						{
							FText(),
							NSLOCTEXT("ListenerLocation", "XDescription", "X-axis of Listener's World Location."),
							FName(),
							{ Inputs::X, 0.0f }
						},
						{
							FText(),
							NSLOCTEXT("ListenerLocation", "YDescription", "Y-axis of Listener's World Location."),
							FName(),
							{ Inputs::Y, 0.0f }
						},
						{
							FText(),
							NSLOCTEXT("ListenerLocation", "ZDescription", "Z-axis of Listener's World Location."),
							FName(),
							{ Inputs::Z, 0.0f }
						}
					};
#if WITH_EDITOR
					Description = LOCTEXT("UE.Listener.Location.Description",
						"Provides information on the world location of the Listener.");
#endif
				}
			};

			static FParameterInterfacePtr InterfacePtr;
			if (!InterfacePtr.IsValid())
			{
				InterfacePtr = MakeShared<FInterface>();
			}

			return InterfacePtr;
		}
	} // namespace ListenerLocationInterface
#undef AUDIO_PARAMETER_INTERFACE_NAMESPACE

#define AUDIO_PARAMETER_INTERFACE_NAMESPACE "UE.Source.Occlusion"
	namespace SourceOcclusionInterface
	{
		const FName Name = AUDIO_PARAMETER_INTERFACE_NAMESPACE;

		namespace Inputs
		{
			const FName IsOccluded = AUDIO_PARAMETER_INTERFACE_MEMBER_DEFINE("IsOccluded");
		}

		Audio::FParameterInterfacePtr GetInterface()
		{
			struct FInterface : public Audio::FParameterInterface
			{
				FInterface()
					: FParameterInterface(SourceOcclusionInterface::Name, { 1, 0 })
				{
					Inputs =
					{
						{
							FText(),
							NSLOCTEXT("IsOccluded", "IsOccludedDescription", "Whether or not this sound is occluded (configure via AttenuationSettings)."),
							FName(),
							{ Inputs::IsOccluded, false }
						}
					};
#if WITH_EDITOR
					Description = LOCTEXT("UE.Source.Occlusion.Description",
						"Provides information around Occlusion as configured by this source's Attenuation Settings.");
#endif
				}
			};

			static FParameterInterfacePtr InterfacePtr;
			if (!InterfacePtr.IsValid())
			{
				InterfacePtr = MakeShared<FInterface>();
			}

			return InterfacePtr;
		}
	} // namespace SourceOcclusionInterface
#undef AUDIO_PARAMETER_INTERFACE_NAMESPACE

#define AUDIO_PARAMETER_INTERFACE_NAMESPACE "UE.World.Time"
	namespace WorldTimeInterface
	{
		const FName Name = AUDIO_PARAMETER_INTERFACE_NAMESPACE;

		namespace Inputs
		{
			const FName WorldTimeWhenPlayed = AUDIO_PARAMETER_INTERFACE_MEMBER_DEFINE("WorldTimeWhenPlayed");
			const FName CurrentWorldTime = AUDIO_PARAMETER_INTERFACE_MEMBER_DEFINE("CurrentWorldTime");
			const FName ElapsedWorldTime = AUDIO_PARAMETER_INTERFACE_MEMBER_DEFINE("ElapsedWorldTime");
		}

		Audio::FParameterInterfacePtr GetInterface()
		{
			struct FInterface : public Audio::FParameterInterface
			{
				FInterface()
					: FParameterInterface(WorldTimeInterface::Name, { 1, 0 })
				{
					Inputs =
					{
						{
							FText(),
							NSLOCTEXT("WorldTime", "WorldTimeWhenPlayedDescription", "[0, X] World time (in seconds) the moment this sound was played."),
							FName(),
							{ Inputs::WorldTimeWhenPlayed, 0.0f }
						},
						{
							FText(),
							NSLOCTEXT("WorldTime", "CurrentWorldTimeDescription", "[0, X] The current value of GameTimeSeconds from the World this sound belongs to."),
							FName(),
							{ Inputs::CurrentWorldTime, 0.0f }
						},
						{
							FText(),
							NSLOCTEXT("WorldTime", "ElapsedWorldTimeDescription", "[0, X] How long (in seconds) this asset has been playing for."),
							FName(),
							{ Inputs::ElapsedWorldTime, 0.0f }
						}
					};
#if WITH_EDITOR
					Description = LOCTEXT("UE.World.Time.Description",
						"Provides information related to the game various time measurements, like how long since this asset was last played or the elapsed time of the World this sound belongs to.");
#endif
				}
			};

			static FParameterInterfacePtr InterfacePtr;
			if (!InterfacePtr.IsValid())
			{
				InterfacePtr = MakeShared<FInterface>();
			}

			return InterfacePtr;
		}
	} // WorldTimeInterface
#undef AUDIO_PARAMETER_INTERFACE_NAMESPACE

} // namespace Audio
#undef LOCTEXT_NAMESPACE
