// Copyright Epic Games, Inc. All Rights Reserved.
#include "Sound/SoundBase.h"

#include "AudioDevice.h"
#include "Engine/AssetUserData.h"
#include "IAudioParameterTransmitter.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/ObjectSaveContext.h"
#include "Sound/SoundSubmix.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SoundBase)

USoundBase::USoundBase(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, VirtualizationMode(EVirtualizationMode::Restart)
	, Duration(-1.0f)
	, Priority(1.0f)
{
#if WITH_EDITORONLY_DATA
	MaxConcurrentPlayCount_DEPRECATED = 16;
#endif // WITH_EDITORONLY_DATA

	//Migrate bOutputToBusOnly settings to Enablement based UI
	bEnableBusSends = true;
	bEnableBaseSubmix = true;
	bEnableSubmixSends = true;
}

bool USoundBase::IsPlayable() const
{
	return false;
}

bool USoundBase::SupportsSubtitles() const
{
	return false;
}

bool USoundBase::HasAttenuationNode() const
{
	return false;
}

const FSoundAttenuationSettings* USoundBase::GetAttenuationSettingsToApply() const
{
	if (AttenuationSettings)
	{
		return &AttenuationSettings->Attenuation;
	}
	return nullptr;
}

float USoundBase::GetMaxDistance() const
{
	if (AttenuationSettings)
	{
		FSoundAttenuationSettings& Settings = AttenuationSettings->Attenuation;
		if (Settings.bAttenuate)
		{
			return Settings.GetMaxDimension();
		}
	}

	return FAudioDevice::GetMaxWorldDistance();
}

float USoundBase::GetDuration() const
{
	return Duration;
}

bool USoundBase::HasDelayNode() const
{
	return bHasDelayNode;
}

bool USoundBase::HasConcatenatorNode() const
{
	return bHasConcatenatorNode;
}

bool USoundBase::IsPlayWhenSilent() const
{
	return VirtualizationMode == EVirtualizationMode::PlayWhenSilent;
}

float USoundBase::GetVolumeMultiplier()
{
	return 1.f;
}

float USoundBase::GetPitchMultiplier()
{
	return 1.f;
}

bool USoundBase::IsOneShot() const
{
	return !IsLooping();
}

bool USoundBase::IsLooping() const
{
	return (GetDuration() >= INDEFINITELY_LOOPING_DURATION);
}

bool USoundBase::ShouldApplyInteriorVolumes()
{
	USoundClass* SoundClass = GetSoundClass();
	return SoundClass && SoundClass->Properties.bApplyAmbientVolumes;
}

USoundClass* USoundBase::GetSoundClass() const
{
	if (SoundClassObject)
	{
		return SoundClassObject;
	}

	const UAudioSettings* AudioSettings = GetDefault<UAudioSettings>();
	if (ensure(AudioSettings))
	{
		if (USoundClass* DefaultSoundClass = AudioSettings->GetDefaultSoundClass())
		{
			return DefaultSoundClass;
		}
	}

	return nullptr;
}

USoundSubmixBase* USoundBase::GetSoundSubmix() const
{
	return SoundSubmixObject;
}

void USoundBase::GetSoundSubmixSends(TArray<FSoundSubmixSendInfo>& OutSends) const
{
	OutSends = SoundSubmixSends;
}

void USoundBase::GetSoundSourceBusSends(EBusSendType BusSendType, TArray<FSoundSourceBusSendInfo>& OutSends) const
{
	if (BusSendType == EBusSendType::PreEffect)
	{
		OutSends = PreEffectBusSends;
	}
	else if (BusSendType == EBusSendType::PostAttenuation)
	{
		OutSends = PostAttenuationBusSends;
	}
	else
	{
		OutSends = BusSends;
	}
}

void USoundBase::GetConcurrencyHandles(TArray<FConcurrencyHandle>& OutConcurrencyHandles) const
{
	const UAudioSettings* AudioSettings = GetDefault<UAudioSettings>();

	OutConcurrencyHandles.Reset();
	if (bOverrideConcurrency)
	{
		OutConcurrencyHandles.Add(ConcurrencyOverrides);
	}
	else if (!ConcurrencySet.IsEmpty())
	{
		for (const USoundConcurrency* Concurrency : ConcurrencySet)
		{
			if (Concurrency)
			{
				OutConcurrencyHandles.Emplace(*Concurrency);
			}
		}
	}
	else if (ensure(AudioSettings))
	{
		if (const USoundConcurrency* DefaultConcurrency = AudioSettings->GetDefaultSoundConcurrency())
		{
			OutConcurrencyHandles.Emplace(*DefaultConcurrency);
		}	
	}
}

float USoundBase::GetPriority() const
{
	return FMath::Clamp(Priority, MIN_SOUND_PRIORITY, MAX_SOUND_PRIORITY);
}

bool USoundBase::GetSoundWavesWithCookedAnalysisData(TArray<USoundWave*>& OutSoundWaves)
{
	return false;
}

#if WITH_EDITOR
void USoundBase::PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	const FName AudioPropertiesSheetFName = GET_MEMBER_NAME_CHECKED(USoundBase, AudioPropertiesSheet);

	if (FProperty* PropertyThatChanged = PropertyChangedEvent.Property)
	{
		const FName& Name = PropertyThatChanged->GetFName();
		if (Name == AudioPropertiesSheetFName)
		{
			BindToPropertySheetChanges();
			InjectPropertySheet();
		}
	}
}

void USoundBase::PreEditChange(FProperty* PropertyAboutToChange)
{
	Super::PreEditChange(PropertyAboutToChange);

	if (!PropertyAboutToChange)
	{
		return;
	}

	if (PropertyAboutToChange->GetName() == GET_MEMBER_NAME_CHECKED(USoundBase, AudioPropertiesSheet) && AudioPropertiesSheet)
	{
		UnbindFromPropertySheetChanges();
	}
}

#endif

#if WITH_EDITORONLY_DATA

void USoundBase::PostLoad()
{
	Super::PostLoad();

	if (bOutputToBusOnly_DEPRECATED)
	{
		bEnableBusSends = true;
		bEnableBaseSubmix = !bOutputToBusOnly_DEPRECATED;
		bEnableSubmixSends = !bOutputToBusOnly_DEPRECATED;
		bOutputToBusOnly_DEPRECATED = false;
	}

	const FPackageFileVersion LinkerUEVersion = GetLinkerUEVersion();

	if (LinkerUEVersion < VER_UE4_SOUND_CONCURRENCY_PACKAGE)
	{
		bOverrideConcurrency = true;
		ConcurrencyOverrides.bLimitToOwner = false;
		ConcurrencyOverrides.SetMaxCount(FMath::Max(MaxConcurrentPlayCount_DEPRECATED, 1));
		ConcurrencyOverrides.ResolutionRule = MaxConcurrentResolutionRule_DEPRECATED;
	}

	if (AudioPropertiesSheet)
	{
		AudioPropertiesSheet->ConditionalPostLoad();
		BindToPropertySheetChanges();
		InjectPropertySheet();
	}
}
#endif // WITH_EDITORONLY_DATA

bool USoundBase::CanBeClusterRoot() const
{
	return false;
}

bool USoundBase::CanBeInCluster() const
{
	return true;
}

void USoundBase::PreSave(FObjectPreSaveContext SaveContext)
{
#if WITH_EDITORONLY_DATA
	InjectPropertySheet();
#endif
	
	Super::PreSave(SaveContext);
}

void USoundBase::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

#if WITH_EDITORONLY_DATA
	if (Ar.IsLoading() || Ar.IsSaving())
	{
		if (SoundConcurrencySettings_DEPRECATED != nullptr)
		{
			ConcurrencySet.Add(SoundConcurrencySettings_DEPRECATED);
			SoundConcurrencySettings_DEPRECATED = nullptr;
		}
	}
#endif // WITH_EDITORONLY_DATA
}

void USoundBase::AddAssetUserData(UAssetUserData* InUserData)
{
	if (InUserData != nullptr)
	{
		UAssetUserData* ExistingData = GetAssetUserDataOfClass(InUserData->GetClass());
		if (ExistingData != nullptr)
		{
			AssetUserData.Remove(ExistingData);
		}
		AssetUserData.Add(InUserData);
	}
}

UAssetUserData* USoundBase::GetAssetUserDataOfClass(TSubclassOf<UAssetUserData> InUserDataClass)
{
	for (int32 DataIdx = 0; DataIdx < AssetUserData.Num(); DataIdx++)
	{
		UAssetUserData* Datum = AssetUserData[DataIdx];
		if (Datum != nullptr && Datum->IsA(InUserDataClass))
		{
			return Datum;
		}
	}
	return nullptr;
}

void USoundBase::RemoveUserDataOfClass(TSubclassOf<UAssetUserData> InUserDataClass)
{
	for (int32 DataIdx = 0; DataIdx < AssetUserData.Num(); DataIdx++)
	{
		UAssetUserData* Datum = AssetUserData[DataIdx];
		if (Datum != nullptr && Datum->IsA(InUserDataClass))
		{
			AssetUserData.RemoveAt(DataIdx);
			return;
		}
	}
}

const TArray<UAssetUserData*>* USoundBase::GetAssetUserDataArray() const
{
	return &ToRawPtrTArrayUnsafe(AssetUserData);
}

TSharedPtr<Audio::IParameterTransmitter> USoundBase::CreateParameterTransmitter(Audio::FParameterTransmitterInitParams&& InParams) const
{
	return nullptr;
}

void USoundBase::InitParameters(TArray<FAudioParameter>& ParametersToInit, FName InFeatureName)
{
	for (int32 i = ParametersToInit.Num() - 1; i >= 0; --i)
	{
		if (!IsParameterValid(ParametersToInit[i]))
		{
			ParametersToInit.RemoveAtSwap(i, EAllowShrinking::No);
		}
	}
}

bool USoundBase::IsParameterValid(const FAudioParameter& InParameter) const
{
	if (InParameter.ParamName.IsNone())
	{
		return false;
	}

	return !(InParameter.ParamType == EAudioParameterType::None || InParameter.ParamType == EAudioParameterType::NoneArray);
}

#if WITH_EDITORONLY_DATA

void USoundBase::SetTimecodeOffset(const FSoundTimecodeOffset& InTimecodeOffset)
{
	TimecodeOffset = InTimecodeOffset;
}

TOptional<FSoundTimecodeOffset> USoundBase::GetTimecodeOffset() const
{
	static const FSoundTimecodeOffset Defaults;
	if(TimecodeOffset == Defaults)
	{
		return {};
	}
	return TimecodeOffset;
}

void USoundBase::AllowPropertyParsing(const FProperty& TargetProperty)
{
	LocalAudioProperties.RemoveSwap(TargetProperty.GetFName());
}

void USoundBase::IgnorePropertyParsing(const FProperty& PropertyToIgnore)
{
	LocalAudioProperties.AddUnique(PropertyToIgnore.GetFName());
}

bool USoundBase::ShouldParseProperty(const FProperty& TargetProperty) const
{
	return !LocalAudioProperties.Contains(TargetProperty.GetFName());
}

void USoundBase::InjectPropertySheet()
{
	if (AudioPropertiesSheet)
	{
		AudioPropertiesSheet->CopyToObjectProperties(this);
	}
}

void USoundBase::BindToPropertySheetChanges()
{
	if (AudioPropertiesSheet)
	{
		AudioPropertiesSheet->BindPropertiesCopyToSheetChanges(this);
	}
}

void USoundBase::UnbindFromPropertySheetChanges()
{
	if (AudioPropertiesSheet)
	{
		AudioPropertiesSheet->UnbindCopyFromPropertySheetChanges(this);
	}
}

#endif //WITH_EDITORONLY_DATA

float USoundBase::ComputeMaxDistance() const
{
	if (const FSoundAttenuationSettings* Settings = GetAttenuationSettingsToApply())
	{
		if (!Settings->bAttenuate)
		{
			return FAudioDevice::GetMaxWorldDistance();
		}

		const float MaxDimension = Settings->GetMaxDimension();
		if (MaxDimension > UE_KINDA_SMALL_NUMBER)
		{
			return MaxDimension;
		}
	}

	return GetMaxDistance();
}