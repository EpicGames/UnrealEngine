// Copyright Epic Games, Inc. All Rights Reserved.


#include "Sound/SoundClass.h"
#include "Engine/AssetUserData.h"
#include "Engine/Engine.h"
#include "Sound/AudioOutputTarget.h"
#include "UObject/UObjectIterator.h"
#include "AudioDevice.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SoundClass)

#if WITH_EDITOR
#include "Framework/Notifications/NotificationManager.h"
#include "Widgets/Notifications/SNotificationList.h"
#endif // WITH_EDITOR

FSoundClassProperties::FSoundClassProperties()
	: Volume(1.0f)
	, Pitch(1.0f)
	, LowPassFilterFrequency(MAX_FILTER_FREQUENCY)
	, AttenuationDistanceScale(1.0f)
	, LFEBleed(0.0f)
	, VoiceCenterChannelVolume(0.0f)
	, RadioFilterVolume(0.0f)
	, RadioFilterVolumeThreshold(0.0f)
	, bApplyEffects(false)
	, bAlwaysPlay(false)
	, bIsUISound(false)
	, bIsMusic(false)
	, bCenterChannelOnly(false)
	, bApplyAmbientVolumes(false)
	, bReverb(true)
	, Default2DReverbSendAmount(0.0f)
	, OutputTarget(EAudioOutputTarget::Speaker)
	, LoadingBehavior(ESoundWaveLoadingBehavior::Inherited)
	, DefaultSubmix(nullptr)
{
}

/*-----------------------------------------------------------------------------
	USoundClass implementation.
-----------------------------------------------------------------------------*/

#if WITH_EDITOR
TSharedPtr<ISoundClassAudioEditor> USoundClass::SoundClassAudioEditor = nullptr;
#endif

USoundClass::USoundClass(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
#if WITH_EDITORONLY_DATA
	, SoundClassGraph(nullptr)
#endif
{
}

void USoundClass::PostLoad()
{
	Super::PostLoad();

#if WITH_EDITORONLY_DATA
	for (int32 ChildIndex = ChildClasses.Num()-1; ChildIndex >= 0; ChildIndex--)
	{
		if (ChildClasses[ChildIndex] != NULL && ChildClasses[ChildIndex]->GetLinkerUEVersion() < VER_UE4_SOUND_CLASS_GRAPH_EDITOR)
		{
			// first come, first served
			if (ChildClasses[ChildIndex]->ParentClass == nullptr)
			{
				ChildClasses[ChildIndex]->ParentClass = this;
			}
			// if already set, we can't be a parent of this child
			else if (ChildClasses[ChildIndex]->ParentClass != this)
			{
				UE_LOGF(LogAudio, Warning, "SoundClass '%ls' - '%ls' removed from children as '%ls' is its parent.", *GetName(), *ChildClasses[ChildIndex]->GetName(), *ChildClasses[ChildIndex]->ParentClass->GetName());
				ChildClasses.RemoveAt(ChildIndex);
			}
		}
	}
#endif

	// Use the main/default audio device for storing and retrieving sound class properties
	FAudioDeviceManager* AudioDeviceManager = (GEngine ? GEngine->GetAudioDeviceManager() : nullptr);

	// Force the properties to be initialized for this SoundClass on all active audio devices
	if (AudioDeviceManager)
	{
		AudioDeviceManager->RegisterSoundClass(this);
	}
}

#if WITH_EDITOR

TArray<USoundClass*> BackupChildClasses;
ESoundWaveLoadingBehavior BackupLoadingBehavior;

void USoundClass::PreEditChange(FProperty* PropertyAboutToChange)
{
	static const FName NAME_ChildClasses = GET_MEMBER_NAME_CHECKED(USoundClass, ChildClasses);
	static const FName NAME_Properties = GET_MEMBER_NAME_CHECKED(FSoundClassProperties, LoadingBehavior);

	if (PropertyAboutToChange && PropertyAboutToChange->GetFName() == NAME_ChildClasses)
	{
		// Take a copy of the current state of child classes
		BackupChildClasses = ChildClasses;
	}
	else if (PropertyAboutToChange && PropertyAboutToChange->GetFName() == NAME_Properties)
	{
		// Copy the current loading behavior in case it does not pass validation.
		BackupLoadingBehavior = Properties.LoadingBehavior;
	}
}

void USoundClass::PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent)
{
	if (PropertyChangedEvent.Property != NULL)
	{
		static const FName NAME_ChildClasses = GET_MEMBER_NAME_CHECKED(USoundClass, ChildClasses);
		static const FName NAME_ParentClass = GET_MEMBER_NAME_CHECKED(USoundClass, ParentClass);
		static const FName NAME_Properties = GET_MEMBER_NAME_CHECKED(FSoundClassProperties, LoadingBehavior);
		static const FName NAME_AttenuationDistanceScale = GET_MEMBER_NAME_CHECKED(FSoundClassProperties, AttenuationDistanceScale);

		if (PropertyChangedEvent.GetPropertyName() == NAME_ChildClasses)
		{
			// Find child that was changed/added
			for (int32 ChildIndex = 0; ChildIndex < ChildClasses.Num(); ChildIndex++)
			{
				if (ChildClasses[ChildIndex] != NULL && !BackupChildClasses.Contains(ChildClasses[ChildIndex]))
				{
					if (ChildClasses[ChildIndex]->RecurseCheckChild(this))
					{
						// Contains cycle so revert to old layout - launch notification to inform user
						FNotificationInfo Info( NSLOCTEXT("Engine", "UnableToChangeSoundClassChildDueToInfiniteLoopNotification", "Could not change SoundClass child as it would create a loop"));
						Info.ExpireDuration = 5.0f;
						Info.Image = FCoreStyle::Get().GetBrush(TEXT("MessageLog.Error"));
						FSlateNotificationManager::Get().AddNotification(Info);

						ChildClasses = BackupChildClasses;
					}
					else
					{
						// Update parentage
						ChildClasses[ChildIndex]->SetParentClass(this);
					}
					break;
				}
			}

			// Update old child's parent if it has been removed
			for (int32 ChildIndex = 0; ChildIndex < BackupChildClasses.Num(); ChildIndex++)
			{
				if (BackupChildClasses[ChildIndex] != NULL && !ChildClasses.Contains(BackupChildClasses[ChildIndex]))
				{
					BackupChildClasses[ChildIndex]->Modify();
					BackupChildClasses[ChildIndex]->ParentClass = NULL;
				}
			}

			RefreshAllGraphs(false);
		}
		else if (PropertyChangedEvent.GetPropertyName() == NAME_ParentClass)
		{
			// Add this sound class to the parent class if it's not already added
			if (ParentClass)
			{
				bool bIsChildClass = false;
				for (int32 i = 0; i < ParentClass->ChildClasses.Num(); ++i)
				{
					USoundClass* ChildClass = ParentClass->ChildClasses[i];
					if (ChildClass && ChildClass == this)
					{
						bIsChildClass = true;
						break;
					}
				}

				if (!bIsChildClass)
				{
					ParentClass->Modify();
					ParentClass->ChildClasses.Add(this);
				}
			}

			Modify();
			RefreshAllGraphs(false);
		}
		else if (PropertyChangedEvent.GetPropertyName() == NAME_Properties)
		{
			// Until we can check FSoundClassProperties during USoundWave::Serialize, we can't support ForceInline here.
			if (Properties.LoadingBehavior == ESoundWaveLoadingBehavior::ForceInline)
			{
				FNotificationInfo Info(NSLOCTEXT("Engine", "ForceInlineUnavailableOnSoundClass", "Using Force Inline on soundclasses is currently not supported. Set each Sound Wave to Force Inline individually instead."));
				Info.ExpireDuration = 5.0f;
				Info.Image = FCoreStyle::Get().GetBrush(TEXT("MessageLog.Error"));
				FSlateNotificationManager::Get().AddNotification(Info);

				Properties.LoadingBehavior = BackupLoadingBehavior;
			}
		}
		else if (PropertyChangedEvent.GetPropertyName() == NAME_AttenuationDistanceScale)
		{
			if (FAudioDeviceManager* AudioDeviceManager = FAudioDeviceManager::Get())
			{
				AudioDeviceManager->IterateOverAllDevices([this](Audio::FDeviceId DeviceId, FAudioDevice* InDevice)
				{
					InDevice->SetSoundClassDistanceScale(this, Properties.AttenuationDistanceScale, 0.2f);
				});
			}
		}
	}

	if (PropertyChangedEvent.ChangeType != EPropertyChangeType::Interactive && PropertyChangedEvent.MemberProperty)
	{
		if (PropertyChangedEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(USoundClass,PassiveSoundMixModifiers))
		{
			TArray<USoundClass*> ProblemClasses;
			for (int32 Index = 0; Index < PassiveSoundMixModifiers.Num(); Index++)
			{
				const FPassiveSoundMixModifier& CurrentSoundMix = PassiveSoundMixModifiers[Index];

				// there may be many dependency loops but we're only concerned with the Sound Class being edited
				if (CurrentSoundMix.SoundMix && CurrentSoundMix.SoundMix->CausesPassiveDependencyLoop(ProblemClasses)
					&& ProblemClasses.Contains(this))
				{
					FFormatNamedArguments Arguments;
					Arguments.Add(TEXT("SoundClass"), FText::FromString(GetName()));
					Arguments.Add(TEXT("SoundMix"), FText::FromString(CurrentSoundMix.SoundMix->GetName()));
					FNotificationInfo Info(FText::Format(NSLOCTEXT("Engine", "PassiveSoundMixLoop", "Passive dependency created by Sound Class'{SoundClass}' and Sound Mix'{SoundMix}' - results may be undesirable"), Arguments));
					Info.ExpireDuration = 10.0f;
					Info.Image = FCoreStyle::Get().GetBrush(TEXT("MessageLog.Warning"));
					FSlateNotificationManager::Get().AddNotification(Info);
				}
			}
		}
	}

	// Use the main/default audio device for storing and retrieving sound class properties
	FAudioDeviceManager* AudioDeviceManager = (GEngine ? GEngine->GetAudioDeviceManager() : nullptr);

	// Force the properties to be initialized for this SoundClass on all active audio devices
	if (AudioDeviceManager)
	{
		AudioDeviceManager->RegisterSoundClass(this);
	}

	Super::PostEditChangeProperty(PropertyChangedEvent);
}
#endif

void USoundClass::Interpolate( float InterpValue, FSoundClassProperties& Current, const FSoundClassProperties& Start, const FSoundClassProperties& End )
{
	if( InterpValue >= 1.0f )
	{
		Current = End;
	}
	else if( InterpValue <= 0.0f )
	{
		Current = Start;
	}
	else
	{
		const float InvInterpValue = 1.0f - InterpValue;

		Current.Volume = ( Start.Volume * InvInterpValue ) + ( End.Volume * InterpValue );
		Current.Pitch = ( Start.Pitch * InvInterpValue ) + ( End.Pitch * InterpValue );
		Current.VoiceCenterChannelVolume = ( Start.VoiceCenterChannelVolume * InvInterpValue ) + ( End.VoiceCenterChannelVolume * InterpValue );
		Current.RadioFilterVolume = ( Start.RadioFilterVolume * InvInterpValue ) + ( End.RadioFilterVolume * InterpValue );
		Current.RadioFilterVolumeThreshold = ( Start.RadioFilterVolumeThreshold * InvInterpValue ) + ( End.RadioFilterVolumeThreshold * InterpValue );
	}
}

void USoundClass::Serialize( FArchive& Ar )
{
	Super::Serialize( Ar );

	if (Ar.UEVer() < VER_UE4_SOUND_CLASS_GRAPH_EDITOR)
	{
		// load this to match size and then throw away
		TMap<USoundClass*, FSoundClassEditorData>	EditorData_DEPRECATED;
		Ar << EditorData_DEPRECATED;
	}

#if WITH_EDITORONLY_DATA
	if (Ar.IsLoading())
	{
		Properties.ModulationSettings.VersionModulators();
	}
#endif // WITH_EDITORONLY_DATA
}

void USoundClass::BeginDestroy()
{
	Super::BeginDestroy();

	if (!GExitPurge && GEngine && GEngine->GetAudioDeviceManager())
	{
		GEngine->GetAudioDeviceManager()->UnregisterSoundClass(this);
	}
}

FString USoundClass::GetDesc( void )
{
	return( FString::Printf( TEXT( "Children: %d" ), ChildClasses.Num() ) );
}

void USoundClass::AddAssetUserData(UAssetUserData* InUserData)
{
	if (InUserData != nullptr)
	{
		UAssetUserData* ExistingData = GetAssetUserDataOfClass(InUserData->GetClass());
		if (ExistingData != nullptr)
		{
			AssetUserData.Remove(ExistingData);
		}
		AssetUserData.Add(InUserData);
	}
}

UAssetUserData* USoundClass::GetAssetUserDataOfClass(TSubclassOf<UAssetUserData> InUserDataClass)
{
	for (int32 DataIdx = 0; DataIdx < AssetUserData.Num(); DataIdx++)
	{
		UAssetUserData* Datum = AssetUserData[DataIdx];
		if (Datum != nullptr && Datum->IsA(InUserDataClass))
		{
			return Datum;
		}
	}
	return nullptr;
}

void USoundClass::RemoveUserDataOfClass(TSubclassOf<UAssetUserData> InUserDataClass)
{
	for (int32 DataIdx = 0; DataIdx < AssetUserData.Num(); DataIdx++)
	{
		UAssetUserData* Datum = AssetUserData[DataIdx];
		if (Datum != nullptr && Datum->IsA(InUserDataClass))
		{
			AssetUserData.RemoveAt(DataIdx);
			return;
		}
	}
}

const TArray<UAssetUserData*>* USoundClass::GetAssetUserDataArray() const
{
	return &ToRawPtrTArrayUnsafe(AssetUserData);
}

#if WITH_EDITOR
bool USoundClass::RecurseCheckChild( USoundClass* ChildSoundClass )
{
	for( int32 Index = 0; Index < ChildClasses.Num(); Index++ )
	{
		if (ChildClasses[Index])
		{
			if( ChildClasses[ Index ] == ChildSoundClass )
			{
				return( true );
			}

			if( ChildClasses[ Index ]->RecurseCheckChild( ChildSoundClass ) )
			{
				return( true );
			}
		}
	}

	return( false );
}

void USoundClass::SetParentClass( USoundClass* InParentClass )
{
	if (ParentClass != InParentClass)
	{
		if (ParentClass != NULL)
		{
			ParentClass->Modify();
			ParentClass->ChildClasses.Remove(this);
		}

		Modify();
		ParentClass = InParentClass;
	}
}

void USoundClass::AddReferencedObjects(UObject* InThis, FReferenceCollector& Collector)
{
	USoundClass* This = CastChecked<USoundClass>(InThis);

	Collector.AddReferencedObject(This->SoundClassGraph, This);

	Super::AddReferencedObjects(InThis, Collector);
}

void USoundClass::RefreshAllGraphs(bool bIgnoreThis)
{
	if (SoundClassAudioEditor.IsValid())
	{
	// Update the graph representation of every SoundClass
	for (TObjectIterator<USoundClass> It; It; ++It)
	{
		USoundClass* SoundClass = *It;
		if (!bIgnoreThis || SoundClass != this)
		{
			if (SoundClass->SoundClassGraph)
			{
					SoundClassAudioEditor->RefreshGraphLinks(SoundClass->SoundClassGraph);
				}
			}
		}
	}
}

void USoundClass::SetSoundClassAudioEditor(TSharedPtr<ISoundClassAudioEditor> InSoundClassAudioEditor)
{
	check(!SoundClassAudioEditor.IsValid());
	SoundClassAudioEditor = InSoundClassAudioEditor;
}

TSharedPtr<ISoundClassAudioEditor> USoundClass::GetSoundClassAudioEditor()
{
	return SoundClassAudioEditor;
}

#endif

