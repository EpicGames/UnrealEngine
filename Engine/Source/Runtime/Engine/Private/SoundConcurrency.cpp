// Copyright Epic Games, Inc. All Rights Reserved.

#include "Sound/SoundConcurrency.h"

#include "Algo/AllOf.h"
#include "AudioDevice.h"
#include "Audio/AudioTraceUtil.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SoundConcurrency)

DEFINE_LOG_CATEGORY(LogAudioConcurrency);

static float ConcurrencyMinVolumeScaleCVar = 1.e-3f;
FAutoConsoleVariableRef CVarConcurrencyMinVolumeScale(
	TEXT("au.Concurrency.MinVolumeScale"),
	ConcurrencyMinVolumeScaleCVar,
	TEXT("Volume threshold considered silent for volume scaling (linear scale).\n"),
	ECVF_Default);


static float ConcurrencyMinDistanceDeltaCVar = 0.0f;
FAutoConsoleVariableRef CVarConcurrencyMinDistanceDelta(
	TEXT("au.Concurrency.MinDistanceDelta"),
	ConcurrencyMinDistanceDeltaCVar,
	TEXT("Minimum distance delta a sound needs to be to determine if it's further away from another sound.\n"),
	ECVF_Default);


namespace SoundConcurrencyPrivate
{
	static const int32 DefaultMaxCount = 16;

	void SetSoundDataTarget(const FActiveSound& ActiveSound, FConcurrencySoundData& SoundData, float InTargetVolume, float InLerpTime)
	{
#if UE_BUILD_SHIPPING
		SoundData.SetTarget(InTargetVolume, InLerpTime);
#else
		const float LastTargetVolume = SoundData.GetTargetVolume();

		SoundData.SetTarget(InTargetVolume, InLerpTime);
		if (!FMath::IsNearlyEqual(LastTargetVolume, InTargetVolume))
		{
			if (const USoundBase* Sound = ActiveSound.GetSound())
			{
				UE_LOGF(LogAudioConcurrency, Verbose,
					"Sound '%ls' concurrency generation '%i' target volume update: %.3f to %.3f.",
					*Sound->GetName(),
					SoundData.Generation,
					LastTargetVolume,
					InTargetVolume);
			}
		}
#endif // UE_BUILD_SHIPPING
	}
} // namespace SoundConcurrencyPrivate


USoundConcurrency::USoundConcurrency(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

bool USoundConcurrency::SetMaxCount(int32 InMaxCount)
{
	return Concurrency.SetMaxCount(InMaxCount);
}

FSoundConcurrencySettings::FSoundConcurrencySettings()
	: MaxCount(SoundConcurrencyPrivate::DefaultMaxCount)
	, PlatformMaxCount(SoundConcurrencyPrivate::DefaultMaxCount)
{
}

int32 FSoundConcurrencySettings::GetMaxCount() const
{
	if (bEnableMaxCountPlatformScaling)
	{
		if (const int32 Count = PlatformMaxCount.GetValue(); Count > 0)
		{
			return Count;
		}
	}
	else
	{
		if (MaxCount > 0)
		{
			return MaxCount;
		}
	}

	return SoundConcurrencyPrivate::DefaultMaxCount;
}

bool FSoundConcurrencySettings::SetMaxCount(int32 InMaxCount)
{
	if (InMaxCount <= 0 || bEnableMaxCountPlatformScaling)
	{
		return false;
	}

	MaxCount = InMaxCount;
	return true;
}

#if WITH_EDITOR
void FSoundConcurrencySettings::SetEnableMaxCountPlatformScaling(bool bEnabled)
{
	bEnableMaxCountPlatformScaling = bEnabled ? 1 : 0;
}

bool FSoundConcurrencySettings::IsMaxCountPlatformScalingEnabled() const
{
	return bEnableMaxCountPlatformScaling > 0;
}

bool FSoundConcurrencySettings::SetPlatformMaxCount(FPerPlatformInt InMaxCount)
{
	bool bIsMaxCountValid = InMaxCount.Default > 0;

#if WITH_EDITORONLY_DATA
	auto IsValidMaxCount = [](const TPair<FName, int32>& Pair) { return Pair.Value > 0; };
	bIsMaxCountValid &= Algo::AllOf(InMaxCount.PerPlatform, IsValidMaxCount);
#endif // WITH_EDITORONLY_DATA

	if (bIsMaxCountValid)
	{
		PlatformMaxCount = MoveTemp(InMaxCount);
		return true;
	}

	return false;
}
#endif // WITH_EDITOR

float FSoundConcurrencySettings::GetVolumeScale() const
{
	return FMath::Clamp(VolumeScale, 0.0f, 1.0f);
}

bool FSoundConcurrencySettings::IsEvictionSupported() const
{
	switch (ResolutionRule)
	{
		case EMaxConcurrentResolutionRule::StopLowestPriority:
		case EMaxConcurrentResolutionRule::StopLowestPriorityThenPreventNew:
		case EMaxConcurrentResolutionRule::StopQuietest:
		{
			return false;
		}

		case EMaxConcurrentResolutionRule::PreventNew:
		case EMaxConcurrentResolutionRule::StopOldest:
		case EMaxConcurrentResolutionRule::StopFarthestThenPreventNew:
		case EMaxConcurrentResolutionRule::StopFarthestThenOldest:
		default:
		{
			static_assert(static_cast<int32>(EMaxConcurrentResolutionRule::Count) == 7,
				"Possible Missing EMaxConcurrentResolutionRule switch case coverage");
			return true;
		}
	}
}

FConcurrencyHandle::FConcurrencyHandle(const FSoundConcurrencySettings& InSettings)
	: Settings(InSettings)
	, ObjectID(0)
	, bIsOverride(true)
{
}

FConcurrencyHandle::FConcurrencyHandle(const USoundConcurrency& Concurrency)
	: Settings(Concurrency.Concurrency)
	, ObjectID(Concurrency.GetUniqueID())
	, bIsOverride(false)
{
}

EConcurrencyMode FConcurrencyHandle::GetMode(const FActiveSound& ActiveSound) const
{
	if (Settings.bLimitToOwner && ActiveSound.GetOwnerID() != 0)
	{
		return bIsOverride && ActiveSound.GetSound() != nullptr
			? EConcurrencyMode::OwnerPerSound
			: EConcurrencyMode::Owner;
	}

	return ObjectID == 0 ? EConcurrencyMode::Sound : EConcurrencyMode::Group;
}

void FConcurrencySoundData::Update(float InElapsed)
{
	Elapsed += InElapsed;
}

float FConcurrencySoundData::GetLerpTime() const
{
	return LerpTime;
}

float FConcurrencySoundData::GetVolume(bool bInDecibels) const
{
	const float Alpha = FMath::Clamp(Elapsed / FMath::Max(LerpTime, UE_SMALL_NUMBER), 0.0f, 1.0f);
	const float VolumeDb = FMath::Lerp(DbStartVolume, DbTargetVolume, Alpha);

	if (bInDecibels)
	{
		return VolumeDb;
	}
	
	float VolumeLin = Audio::ConvertToLinear(VolumeDb);
	if (VolumeLin < ConcurrencyMinVolumeScaleCVar || FMath::IsNearlyEqual(VolumeLin, ConcurrencyMinVolumeScaleCVar))
	{
		return 0.0f;
	}
	return VolumeLin;
}

float FConcurrencySoundData::GetTargetVolume(bool bInDecibels) const
{
	if (bInDecibels)
	{
		return DbTargetVolume;
	}

	const float VolumeLin = Audio::ConvertToLinear(DbTargetVolume);
	if (VolumeLin < ConcurrencyMinVolumeScaleCVar || FMath::IsNearlyEqual(VolumeLin, ConcurrencyMinVolumeScaleCVar))
	{
		return 0.0f;
	}

	return VolumeLin;
}

void FConcurrencySoundData::SetTarget(float InTargetVolume, float InLerpTime)
{
	DbStartVolume = GetVolume(true /* bInDecibels */);
	DbTargetVolume = Audio::ConvertToDecibels(InTargetVolume, ConcurrencyMinVolumeScaleCVar);
	LerpTime = FMath::Max(InLerpTime, 0.0f);
	Elapsed = 0.0f;
}

FConcurrencyGroup::FConcurrencyGroup(FConcurrencyGroupID InGroupID, const FConcurrencyHandle& ConcurrencyHandle)
	: GroupID(InGroupID)
	, ObjectID(ConcurrencyHandle.ObjectID)
	, Settings(ConcurrencyHandle.Settings)
	, LastTimePlayed(FPlatformTime::ToSeconds64(FPlatformTime::Cycles64()))
{
}

FConcurrencyGroupID FConcurrencyGroup::GenerateNewID()
{
	static FConcurrencyGroupID ConcurrencyGroupIDs = 0;
	return ++ConcurrencyGroupIDs;
}

const TArray<FActiveSound*>& FConcurrencyGroup::GetActiveSounds() const
{
	return ActiveSounds;
}

FConcurrencyGroupID FConcurrencyGroup::GetGroupID() const
{
	return GroupID;
}

const int32 FConcurrencyGroup::GetNextGeneration() const
{
	return ActiveSounds.Num();
}

const FSoundConcurrencySettings& FConcurrencyGroup::GetSettings() const
{
	return Settings;
}

FConcurrencyObjectID FConcurrencyGroup::GetObjectID() const
{
	return ObjectID;
}

bool FConcurrencyGroup::IsEmpty() const
{
	return ActiveSounds.IsEmpty();
}

bool FConcurrencyGroup::IsFull() const
{
	return Settings.GetMaxCount() <= ActiveSounds.Num();
}

void FConcurrencyGroup::AddActiveSound(FActiveSound& ActiveSound)
{
	check(GroupID != 0);

	if (ActiveSound.ConcurrencyGroupData.Contains(GroupID))
	{
		UE_LOGF(LogAudioConcurrency, Fatal, "Attempting to add active sound '%ls' to concurrency group multiple times.", *ActiveSound.GetOwnerName());
		return;
	}

	FConcurrencySoundData SoundData;

	if (Settings.VolumeScaleMode == EConcurrencyVolumeScaleMode::Default)
	{
		SoundData.Generation = ActiveSounds.Num();
		SoundConcurrencyPrivate::SetSoundDataTarget(ActiveSound, SoundData, 1.0f, 0.0f);
	}

	ActiveSounds.Add(&ActiveSound);
	ActiveSound.ConcurrencyGroupData.Add(GroupID, MoveTemp(SoundData));

	UpdateGeneration(&ActiveSound);
}

void FConcurrencyGroup::RemoveActiveSound(FActiveSound& ActiveSound)
{
	// Remove from array
	const int32 NumRemoved = ActiveSounds.RemoveSwap(&ActiveSound);
	if (NumRemoved == 0)
	{
		return;
	}
	check(NumRemoved == 1);

	FConcurrencySoundData& SoundData = ActiveSound.ConcurrencyGroupData.FindChecked(GroupID);

	// Rebase generations due to removal of a member
	for (FActiveSound* OtherSound : ActiveSounds)
	{
		check(OtherSound);
		FConcurrencySoundData& OtherSoundData = OtherSound->ConcurrencyGroupData.FindChecked(GroupID);
		if (OtherSoundData.Generation > SoundData.Generation)
		{
			check(OtherSoundData.Generation > 0);
			check(OtherSoundData.Generation <= ActiveSounds.Num());
			OtherSoundData.Generation--;
		}

		if (Settings.bVolumeScaleCanRelease && Settings.VolumeScaleMode == EConcurrencyVolumeScaleMode::Default)
		{
			const float GenerationDelta = static_cast<float>(ActiveSounds.Num() - OtherSoundData.Generation) - 1.0f;
			const float NewTargetVolume = FMath::Clamp(FMath::Pow(Settings.GetVolumeScale(), GenerationDelta), 0.0f, 1.0f);
			SoundConcurrencyPrivate::SetSoundDataTarget(*OtherSound, OtherSoundData, NewTargetVolume, Settings.VolumeScaleReleaseTime);
		}
	}
}

void FConcurrencyGroup::UpdateGeneration(FActiveSound* NewActiveSound)
{
	// Default uses value set at time of adding sound to group, so no sort required
	if (Settings.VolumeScaleMode == EConcurrencyVolumeScaleMode::Default)
	{
		return;
	}

	struct FCompareActiveSounds
	{
		EConcurrencyVolumeScaleMode ScaleMode;

		FCompareActiveSounds(EConcurrencyVolumeScaleMode InScaleMode)
			: ScaleMode(InScaleMode)
		{
		}

		FORCEINLINE bool operator()(const FActiveSound& A, const FActiveSound& B) const
		{
			switch(ScaleMode)
			{
				case EConcurrencyVolumeScaleMode::Distance:
				{
					float DistSqA = 0.0f;
					float DistSqB = 0.0f;
					A.AudioDevice->GetDistanceSquaredToNearestListener(A.LastLocation, DistSqA);
					B.AudioDevice->GetDistanceSquaredToNearestListener(B.LastLocation, DistSqB);

					// If sounds share the same distance, newer sounds will be sorted first to avoid volume ping-ponging 
					if (FMath::IsNearlyEqual(DistSqA, DistSqB, UE_KINDA_SMALL_NUMBER))
					{
						return A.GetPlayOrder() < B.GetPlayOrder();
					}

					return DistSqA > DistSqB;
				}

				case EConcurrencyVolumeScaleMode::Priority:
				{
					// Ensures sounds set to always play are sorted above those that aren't, but are sorted appropriately between one another
					const float APriority = A.GetConcurrencyPriority();
					const float BPriority = B.GetConcurrencyPriority();

					// If sounds share the same priority, newer sounds will be sorted last to avoid volume ping-ponging 
					if (FMath::IsNearlyEqual(APriority, BPriority, UE_KINDA_SMALL_NUMBER))
					{
						return A.GetPlayOrder() < B.GetPlayOrder();
					}

					return APriority < BPriority;
				}
			}

			return true;
		}
	};

	ActiveSounds.Sort(FCompareActiveSounds(Settings.VolumeScaleMode));

	for (int32 i = 0; i < ActiveSounds.Num(); ++i)
	{
		FActiveSound* ActiveSound = ActiveSounds[i];
		
		if (FConcurrencySoundData* SoundData = ActiveSound->ConcurrencyGroupData.Find(GetGroupID()))
		{
			SoundData->Generation = i;
			const float GenerationDelta = static_cast<float>(ActiveSounds.Num() - SoundData->Generation) - 1.0f;
			const float NewTargetVolume = FMath::Clamp(FMath::Pow(Settings.GetVolumeScale(), GenerationDelta), 0.0f, 1.0f);

			// If new sound added, immediately lerp to the generation volume.
			const float AttackTime = NewActiveSound == ActiveSound ? 0.0f : Settings.VolumeScaleAttackTime;

			if (!FMath::IsNearlyEqual(AttackTime, SoundData->GetLerpTime()) || !FMath::IsNearlyEqual(SoundData->GetTargetVolume(), NewTargetVolume, ConcurrencyMinVolumeScaleCVar))
			{
				SoundConcurrencyPrivate::SetSoundDataTarget(*ActiveSound, *SoundData, NewTargetVolume, AttackTime);
			}
		}
	}
}

void FConcurrencyGroup::CullSoundsDueToMaxConcurrency()
{
	// Nothing to do if our active sound count is less than or equal to our max active sounds
	// or if eviction is supported
	if (ActiveSounds.Num() <= Settings.GetMaxCount())
	{
		return;
	}

	// Comparator for sorting group's ActiveSounds according to their dynamic concurrency nature.
	// Quieter/lower priority sounds will be at the front of the array.
	struct FCompareActiveSounds
	{
		EMaxConcurrentResolutionRule::Type ResolutionRule;

		FCompareActiveSounds(EMaxConcurrentResolutionRule::Type InResolutionRule)
			: ResolutionRule(InResolutionRule)
		{
		}

		FORCEINLINE bool operator()(const FActiveSound& A, const FActiveSound& B) const
		{
			switch (ResolutionRule)
			{
				case EMaxConcurrentResolutionRule::StopQuietest:
				{
					// If sounds share the same volume, newer sounds will be sorted first to avoid loop realization ping-ponging 
					if (FMath::IsNearlyEqual(A.VolumeConcurrency, B.VolumeConcurrency, UE_KINDA_SMALL_NUMBER))
					{
						return A.GetPlayOrder() > B.GetPlayOrder();
					}
					return A.VolumeConcurrency < B.VolumeConcurrency;
				}

				case EMaxConcurrentResolutionRule::StopLowestPriority:
				case EMaxConcurrentResolutionRule::StopLowestPriorityThenPreventNew:
				{
					// Ensures sounds set to always play are sorted above those that aren't, but are sorted appropriately between one another
					const float APriority = A.GetConcurrencyPriority();
					const float BPriority = B.GetConcurrencyPriority();

					if (!FMath::IsNearlyEqual(APriority, BPriority, UE_KINDA_SMALL_NUMBER))
					{
						return APriority < BPriority;
					}

					// Newer sounds pushed forward in sort to make them more likely to be culled if PreventNew
					return ResolutionRule == EMaxConcurrentResolutionRule::StopLowestPriorityThenPreventNew
						? A.GetPlayOrder() > B.GetPlayOrder()
						: A.GetPlayOrder() < B.GetPlayOrder();
				}

				default:
				case EMaxConcurrentResolutionRule::PreventNew:
				case EMaxConcurrentResolutionRule::StopOldest:
				case EMaxConcurrentResolutionRule::StopFarthestThenPreventNew:
				case EMaxConcurrentResolutionRule::StopFarthestThenOldest:
				{
					static_assert(static_cast<int32>(EMaxConcurrentResolutionRule::Count) == 7,
						"Possible Missing EMaxConcurrentResolutionRule switch case coverage");
					return true;
				}
			}
		}
	};

	ActiveSounds.Sort(FCompareActiveSounds(Settings.ResolutionRule));

	// Need to make a new list when stopping the sounds since the process of stopping an active sound
	// will remove the sound from this concurrency group's ActiveSounds array.
	const int32 NumSoundsToStop = ActiveSounds.Num() - Settings.GetMaxCount();

	check(NumSoundsToStop > 0);
	for (int32 i = 0; i < ActiveSounds.Num(); ++i)
	{
		FActiveSound* ActiveSound = ActiveSounds[i];
		check(ActiveSound);

		const bool bShouldStop = i < NumSoundsToStop;
		ActiveSound->bShouldStopDueToMaxConcurrency = bShouldStop;
	}
}

bool FConcurrencyGroup::CanPlaySoundNow(float InCurrentTime) const
{
	if (Settings.RetriggerTime > 0.0f)
	{
		const float DeltaTime = InCurrentTime - LastTimePlayed;
		if (DeltaTime < Settings.RetriggerTime)
		{
			UE_LOGF(LogAudioConcurrency, VeryVerbose, "Rejected Sound for Group ID (%d) with DeltaTime: %.2f", GroupID, DeltaTime);
			return false;
		}
	}
	return true;
}

FSoundConcurrencyManager::FSoundConcurrencyManager(class FAudioDevice* InAudioDevice)
	: AudioDevice(InAudioDevice)
{
}

FSoundConcurrencyManager::~FSoundConcurrencyManager()
{
}

void FSoundConcurrencyManager::CreateNewGroupsFromHandles(
	const FActiveSound& NewActiveSound,
	const TArray<FConcurrencyHandle>& ConcurrencyHandles,
	TArray<FConcurrencyGroup*>& OutGroupsToApply
)
{
	for (const FConcurrencyHandle& ConcurrencyHandle : ConcurrencyHandles)
	{
		switch (ConcurrencyHandle.GetMode(NewActiveSound))
		{
			case EConcurrencyMode::Group:
			{
				if (!ConcurrencyMap.Contains(ConcurrencyHandle.ObjectID))
				{
					FConcurrencyGroup& ConcurrencyGroup = CreateNewConcurrencyGroup(ConcurrencyHandle);
					ConcurrencyMap.Add(ConcurrencyHandle.ObjectID, ConcurrencyGroup.GetGroupID());
					OutGroupsToApply.Add(&ConcurrencyGroup);
				}
			}
			break;

			case EConcurrencyMode::Owner:
			{
				const FSoundOwnerObjectID OwnerObjectID = NewActiveSound.GetOwnerID();
				if (FOwnerConcurrencyMapEntry* ConcurrencyEntry = OwnerConcurrencyMap.Find(OwnerObjectID))
				{
					if (!ConcurrencyEntry->ConcurrencyObjectToConcurrencyGroup.Contains(ConcurrencyHandle.ObjectID))
					{
						FConcurrencyGroup& ConcurrencyGroup = CreateNewConcurrencyGroup(ConcurrencyHandle);
						ConcurrencyEntry->ConcurrencyObjectToConcurrencyGroup.Add(ConcurrencyHandle.ObjectID, ConcurrencyGroup.GetGroupID());
						OutGroupsToApply.Add(&ConcurrencyGroup);
					}
				}
				else
				{
					FConcurrencyGroup& ConcurrencyGroup = CreateNewConcurrencyGroup(ConcurrencyHandle);
					OwnerConcurrencyMap.Emplace(OwnerObjectID, FOwnerConcurrencyMapEntry(ConcurrencyHandle.ObjectID, ConcurrencyGroup.GetGroupID()));
					OutGroupsToApply.Add(&ConcurrencyGroup);
				}
			}
			break;

			case EConcurrencyMode::OwnerPerSound:
			{
				USoundBase* Sound = NewActiveSound.GetSound();
				check(Sound);

				const FSoundObjectID SoundObjectID = static_cast<FSoundObjectID>(Sound->GetUniqueID());
				const FSoundOwnerObjectID OwnerObjectID = NewActiveSound.GetOwnerID();

				if (FSoundInstanceEntry* InstanceEntry = OwnerPerSoundConcurrencyMap.Find(OwnerObjectID))
				{
					if (!InstanceEntry->SoundInstanceToConcurrencyGroup.Contains(SoundObjectID))
					{
						FConcurrencyGroup& ConcurrencyGroup = CreateNewConcurrencyGroup(ConcurrencyHandle);
						InstanceEntry->SoundInstanceToConcurrencyGroup.Add(SoundObjectID, ConcurrencyGroup.GetGroupID());
						OutGroupsToApply.Add(&ConcurrencyGroup);
					}
				}
				else
				{
					FConcurrencyGroup& ConcurrencyGroup = CreateNewConcurrencyGroup(ConcurrencyHandle);
					OwnerPerSoundConcurrencyMap.Emplace(OwnerObjectID, FSoundInstanceEntry(SoundObjectID, ConcurrencyGroup.GetGroupID()));
					OutGroupsToApply.Add(&ConcurrencyGroup);
				}
			}
			break;

			case EConcurrencyMode::Sound:
			{
				const FSoundObjectID SoundObjectID = NewActiveSound.GetSound()->GetUniqueID();
				if (!SoundObjectToConcurrencyGroup.Contains(SoundObjectID))
				{
					FConcurrencyGroup& ConcurrencyGroup = CreateNewConcurrencyGroup(ConcurrencyHandle);
					SoundObjectToConcurrencyGroup.Add(SoundObjectID, ConcurrencyGroup.GetGroupID());
					OutGroupsToApply.Add(&ConcurrencyGroup);
				}
			}
			break;
		}
	}
}

FActiveSound* FSoundConcurrencyManager::CreateNewActiveSound(const FActiveSound& NewActiveSound, bool bIsRetriggering)
{
	check(NewActiveSound.GetSound());
	check(IsInAudioThread());

	// If there are no concurrency settings associated then there is no limit on this sound
	TArray<FConcurrencyHandle> ConcurrencyHandles;
	NewActiveSound.GetConcurrencyHandles(ConcurrencyHandles);

	// If there was no concurrency or the setting was zero, then always play this sound.
	if (!ConcurrencyHandles.Num())
	{
		FActiveSound* ActiveSound = new FActiveSound(NewActiveSound);
		ActiveSound->PlaybackTimeNonVirtualized = 0.0f;
		ActiveSound->SetAudioDevice(AudioDevice);
		return ActiveSound;
	}

#if !UE_BUILD_SHIPPING
	for (const FConcurrencyHandle& ConcurrencyHandle : ConcurrencyHandles)
	{
		check(ConcurrencyHandle.Settings.GetMaxCount() > 0);
	}
#endif

	return EvaluateConcurrency(NewActiveSound, ConcurrencyHandles, bIsRetriggering);
}

FConcurrencyGroup& FSoundConcurrencyManager::CreateNewConcurrencyGroup(const FConcurrencyHandle& ConcurrencyHandle)
{
	//Create & add new concurrency group to the map
	FConcurrencyGroupID GroupID = FConcurrencyGroup::GenerateNewID();
	ConcurrencyGroups.Emplace(GroupID, new FConcurrencyGroup(GroupID, ConcurrencyHandle));

	return *ConcurrencyGroups.FindRef(GroupID);
}

FConcurrencyGroup* FSoundConcurrencyManager::CanPlaySound(const FActiveSound& NewActiveSound, const FConcurrencyGroupID GroupID, TArray<FActiveSound*>& OutSoundsToEvict, bool bIsRetriggering)
{
	check(GroupID != 0);
	FConcurrencyGroup* ConcurrencyGroup = ConcurrencyGroups.FindRef(GroupID);
	if (!ConcurrencyGroup)
	{
		UE_LOGF(LogAudioConcurrency, Warning, "Attempting to add active sound '%ls' (owner '%ls') to invalid concurrency group.",
			NewActiveSound.GetSound() ? *NewActiveSound.GetSound()->GetFullName() : TEXT("Unset"),
			*NewActiveSound.GetOwnerName());
		return nullptr;
	}

	// Check for rate limiting behavior
 	const float CurrentTime = FPlatformTime::ToSeconds64(FPlatformTime::Cycles64());
	if (!ConcurrencyGroup->CanPlaySoundNow(CurrentTime))
	{
		return nullptr;
	}

	// If we have successfully played a sound, then we update the last time a sound played here
	ConcurrencyGroup->SetLastTimePlayed(CurrentTime);

	// Some settings don't support immediate eviction, they cull once we instantiate the sound.  This
	// is because it is not possible to evaluate sound volumes/priorities/etc *before* they play.
	if ( !ConcurrencyGroup->GetSettings().IsEvictionSupported() )
	{
		return ConcurrencyGroup;
	}

	if (ConcurrencyGroup->IsFull())
	{
		// If no room for new sound, early out
		if (FActiveSound* SoundToEvict = GetEvictableSound(NewActiveSound, *ConcurrencyGroup, bIsRetriggering))
		{
			OutSoundsToEvict.AddUnique(SoundToEvict);
		}
		else
		{
			return nullptr;
		}
	}

	return ConcurrencyGroup;
}

FActiveSound* FSoundConcurrencyManager::GetEvictableSound(const FActiveSound& NewActiveSound, const FConcurrencyGroup& ConcurrencyGroup, bool bIsRetriggering)
{
	check(ConcurrencyGroup.IsFull());

	const EMaxConcurrentResolutionRule::Type Rule = ConcurrencyGroup.GetSettings().ResolutionRule;
	switch (Rule)
	{
		case EMaxConcurrentResolutionRule::PreventNew:
		{
			return nullptr;
		}

		case EMaxConcurrentResolutionRule::StopOldest:
		{
			return GetEvictableSoundStopOldest(NewActiveSound, ConcurrencyGroup, bIsRetriggering);

		}

		case EMaxConcurrentResolutionRule::StopFarthestThenPreventNew:
		case EMaxConcurrentResolutionRule::StopFarthestThenOldest:
		{
			return GetEvictableSoundStopFarthest(NewActiveSound, ConcurrencyGroup, bIsRetriggering);
		}

		// Eviction not supported by following rules due to requiring the sound to be initialized in order to calculate.
		// Therefore, it is culled later and not evicted.
		case EMaxConcurrentResolutionRule::StopLowestPriority:
		case EMaxConcurrentResolutionRule::StopLowestPriorityThenPreventNew:
		case EMaxConcurrentResolutionRule::StopQuietest:
		{
			return nullptr;
		}

		default:
		{
			static_assert(static_cast<int32>(EMaxConcurrentResolutionRule::Count) == 7,
				"Possible Missing EMaxConcurrentResolutionRule switch case coverage");
		}
		break;
	}

	return nullptr;
}

FActiveSound* FSoundConcurrencyManager::GetEvictableSoundStopOldest(const FActiveSound& NewActiveSound, const FConcurrencyGroup& ConcurrencyGroup, bool bIsRetriggering) const
{
	FActiveSound* EvictableSound = nullptr;
	const TArray<FActiveSound*>& ActiveSounds = ConcurrencyGroup.GetActiveSounds();
	for (FActiveSound* ActiveSound : ActiveSounds)
	{
		if (EvictableSound == nullptr || ActiveSound->PlaybackTime > EvictableSound->PlaybackTime)
		{
			EvictableSound = ActiveSound;
		}
	}

	// Don't evict if attempting to re-trigger an older sound than that which is currently playing
	if (bIsRetriggering && EvictableSound && NewActiveSound.PlaybackTime > EvictableSound->PlaybackTime)
	{
		return nullptr;
	}

	return EvictableSound;
}

FActiveSound* FSoundConcurrencyManager::GetEvictableSoundStopFarthest(const FActiveSound& NewActiveSound, const FConcurrencyGroup& ConcurrencyGroup, bool bIsRetriggering) const
{
	const EMaxConcurrentResolutionRule::Type Rule = ConcurrencyGroup.GetSettings().ResolutionRule;

	// Get the mininum distance used to calculate distance of a sound
	float MinDistanceSquared = FMath::Max(ConcurrencyMinDistanceDeltaCVar, 0.0f);
	MinDistanceSquared *= MinDistanceSquared;

	check(AudioDevice);
	float DistanceToStopSoundSq = 0.0f;
	const bool bAllowAttenuationOverrides = true;
	int32 ClosestListenerIndex = AudioDevice->FindClosestListenerIndex(NewActiveSound.Transform.GetTranslation(), DistanceToStopSoundSq, bAllowAttenuationOverrides);

	FActiveSound* EvictableSound = nullptr;
	const TArray<FActiveSound*>& ActiveSounds = ConcurrencyGroup.GetActiveSounds();
	for (FActiveSound* ActiveSound : ActiveSounds)
	{
		check(ActiveSound);

		float DistanceToActiveSoundSq;
		ClosestListenerIndex = AudioDevice->FindClosestListenerIndex(ActiveSound->Transform.GetTranslation(), DistanceToActiveSoundSq, bAllowAttenuationOverrides);


		// Stop Farthest
		if (DistanceToActiveSoundSq > (DistanceToStopSoundSq + MinDistanceSquared))
		{
				DistanceToStopSoundSq = DistanceToActiveSoundSq;
				EvictableSound = ActiveSound;
				continue;
		}

		// Stop Farthest then Oldest
		if (Rule == EMaxConcurrentResolutionRule::StopFarthestThenOldest)
		{
			const bool bIsSameDistance = FMath::Abs(DistanceToActiveSoundSq - DistanceToStopSoundSq) <= MinDistanceSquared;
			if (bIsSameDistance)
			{
				const bool bIsOlderThanChosen = EvictableSound == nullptr || ActiveSound->PlaybackTime > EvictableSound->PlaybackTime;
				if (bIsOlderThanChosen)
				{
					// Don't evict if attempting to re-trigger an older sound than that which is currently playing
					if (!bIsRetriggering || ActiveSound->PlaybackTime > NewActiveSound.PlaybackTime)
					{
						DistanceToStopSoundSq = DistanceToActiveSoundSq;
						EvictableSound = ActiveSound;
					}
				}
			}
		}
	}

	return EvictableSound;
}

FActiveSound* FSoundConcurrencyManager::EvaluateConcurrency(const FActiveSound& NewActiveSound, TArray<FConcurrencyHandle>& ConcurrencyHandles, bool bIsRetriggering)
{
	check(NewActiveSound.GetSound());

	TArray<FActiveSound*> SoundsToEvict;
	TArray<FConcurrencyGroup*> GroupsToApply;

	for (const FConcurrencyHandle& ConcurrencyHandle : ConcurrencyHandles)
	{
		switch (ConcurrencyHandle.GetMode(NewActiveSound))
		{
			case EConcurrencyMode::Group:
			{
				if (FConcurrencyGroupID* ConcurrencyGroupID = ConcurrencyMap.Find(ConcurrencyHandle.ObjectID))
				{
					FConcurrencyGroup* ConcurrencyGroup = CanPlaySound(NewActiveSound, *ConcurrencyGroupID, SoundsToEvict, bIsRetriggering);
					if (!ConcurrencyGroup)
					{
						return nullptr;
					}
					GroupsToApply.Add(ConcurrencyGroup);
				}
			}
			break;

			case EConcurrencyMode::Owner:
			{
				if (FOwnerConcurrencyMapEntry* ConcurrencyEntry = OwnerConcurrencyMap.Find(NewActiveSound.GetOwnerID()))
				{
					if (FConcurrencyGroupID* ConcurrencyGroupID = ConcurrencyEntry->ConcurrencyObjectToConcurrencyGroup.Find(ConcurrencyHandle.ObjectID))
					{
						FConcurrencyGroup* ConcurrencyGroup = CanPlaySound(NewActiveSound, *ConcurrencyGroupID, SoundsToEvict, bIsRetriggering);
						if (!ConcurrencyGroup)
						{
							return nullptr;
						}
						GroupsToApply.Add(ConcurrencyGroup);
					}
				}
			}
			break;

			case EConcurrencyMode::OwnerPerSound:
			{
				const uint32 OwnerObjectID = NewActiveSound.GetOwnerID();
				if (FSoundInstanceEntry* InstanceEntry = OwnerPerSoundConcurrencyMap.Find(OwnerObjectID))
				{
					USoundBase* Sound = NewActiveSound.GetSound();
					check(Sound);
					if (FConcurrencyGroupID* ConcurrencyGroupID = InstanceEntry->SoundInstanceToConcurrencyGroup.Find(Sound->GetUniqueID()))
					{
						FConcurrencyGroup* ConcurrencyGroup = CanPlaySound(NewActiveSound, *ConcurrencyGroupID, SoundsToEvict, bIsRetriggering);
						if (!ConcurrencyGroup)
						{
							return nullptr;
						}
						GroupsToApply.Add(ConcurrencyGroup);
					}
				}
			}
			break;

			case EConcurrencyMode::Sound:
			{
				const FSoundObjectID SoundObjectID = NewActiveSound.GetSound()->GetUniqueID();
				if (FConcurrencyGroupID* ConcurrencyGroupID = SoundObjectToConcurrencyGroup.Find(SoundObjectID))
				{
					FConcurrencyGroup* ConcurrencyGroup = CanPlaySound(NewActiveSound, *ConcurrencyGroupID, SoundsToEvict, bIsRetriggering);
					if (!ConcurrencyGroup)
					{
						return nullptr;
					}
					GroupsToApply.Add(ConcurrencyGroup);
				}
			}
			break;
		}
	}

	CreateNewGroupsFromHandles(NewActiveSound, ConcurrencyHandles, GroupsToApply);
	return CreateAndEvictActiveSounds(NewActiveSound, GroupsToApply, SoundsToEvict);
}

FActiveSound* FSoundConcurrencyManager::CreateAndEvictActiveSounds(const FActiveSound& NewActiveSound, const TArray<FConcurrencyGroup*>& GroupsToApply, const TArray<FActiveSound*>& SoundsToEvict)
{
	// First make a new active sound
	FActiveSound* ActiveSound = new FActiveSound(NewActiveSound);
	ActiveSound->SetAudioDevice(AudioDevice);
	check(AudioDevice == ActiveSound->AudioDevice);

	bool bTrackConcurrencyVolume = false;
	for (FConcurrencyGroup* ConcurrencyGroup : GroupsToApply)
	{
		check(ConcurrencyGroup);

		const FSoundConcurrencySettings& Settings = ConcurrencyGroup->GetSettings();
		if (Settings.VolumeScaleMode == EConcurrencyVolumeScaleMode::Default)
		{
			const float Volume = Settings.GetVolumeScale();
			if (!FMath::IsNearlyEqual(Volume, 1.0f))
			{
				check(Volume >= 0.0f);
				const int32 NextGeneration = ConcurrencyGroup->GetNextGeneration();

				// If we're ducking older sounds in the concurrency group, then loop through each sound in the concurrency group
				// and update their duck amount based on each sound's generation and the next generation count. The older the sound, the more ducking.
				const TArray<FActiveSound*>& ActiveSounds = ConcurrencyGroup->GetActiveSounds();
				for (FActiveSound* OtherSound : ActiveSounds)
				{
					check(OtherSound);

					FConcurrencySoundData& OtherSoundData = OtherSound->ConcurrencyGroupData.FindChecked(ConcurrencyGroup->GetGroupID());
					const float ActiveSoundGeneration = static_cast<float>(OtherSoundData.Generation);
					const float GenerationDelta = NextGeneration - ActiveSoundGeneration;
					const float NewTargetVolume = FMath::Pow(Volume, GenerationDelta);

					// Don't allow volume to recover if release time is negative
					if (Settings.bVolumeScaleCanRelease && OtherSoundData.GetTargetVolume() < NewTargetVolume)
					{
						continue;
					}

					SoundConcurrencyPrivate::SetSoundDataTarget(*OtherSound, OtherSoundData, NewTargetVolume, Settings.VolumeScaleAttackTime);
				}
			}
		}

		// Determine if we need to track concurrency volume on this active sound
		if (ConcurrencyGroup->GetSettings().ResolutionRule == EMaxConcurrentResolutionRule::StopQuietest)
		{
			bTrackConcurrencyVolume = true;
		}

		// And add it to to the concurrency group.
		ConcurrencyGroup->AddActiveSound(*ActiveSound);
	}

	if (!bTrackConcurrencyVolume)
	{
		ActiveSound->VolumeConcurrency = -1.0f;
	}

	// Stop any sounds now if needed
	for (FActiveSound* SoundToEvict : SoundsToEvict)
	{
		check(SoundToEvict);
		check(AudioDevice == SoundToEvict->AudioDevice);

		// Remove the active sound from the concurrency manager immediately so it doesn't count towards
		// subsequent concurrency resolution checks (i.e. if sounds are triggered multiple times in this frame)
		RemoveActiveSound(*SoundToEvict);

		if (SoundToEvict->FadeOut == FActiveSound::EFadeOut::Concurrency)
		{
			continue;
		}

		if (AudioDevice->IsPendingStop(SoundToEvict))
		{
			continue;
		}

		if (SoundToEvict->GetSound())
		{
			UE_LOGF(LogAudioConcurrency, VeryVerbose, "Evicting Sound %ls due to concurrency", *(SoundToEvict->GetSound()->GetName()));
		}

		StopDueToVoiceStealing(*SoundToEvict);
	}

	return ActiveSound;
}

void FSoundConcurrencyManager::RemoveActiveSound(FActiveSound& ActiveSound)
{
	check(IsInAudioThread());

	// Remove this sound from it's concurrency list
	for (const TPair<FConcurrencyGroupID, FConcurrencySoundData>& SoundDataPair : ActiveSound.ConcurrencyGroupData)
	{
		const FConcurrencyGroupID ConcurrencyGroupID = SoundDataPair.Key;
		FConcurrencyGroup* ConcurrencyGroup = ConcurrencyGroups.FindRef(ConcurrencyGroupID);
		if (!ConcurrencyGroup)
		{
			UE_LOGF(LogAudioConcurrency, Error, "Attempting to remove stopped sound '%ls' from inactive concurrency group.",
				ActiveSound.GetSound() ? *ActiveSound.GetSound()->GetName(): TEXT("Unset"));
			continue;
		}

		check(!ConcurrencyGroup->IsEmpty());
		ConcurrencyGroup->RemoveActiveSound(ActiveSound);

		// Don't delete the concurrency group state if there is a retrigger time set. This is so that 
		// state can persist past the last sound playing in the group
		if (ConcurrencyGroup->IsEmpty() && ConcurrencyGroup->GetSettings().RetriggerTime <= 0.0f)
		{
			// Get the object ID prior to removing from groups collection to avoid reading
			// from the object after its destroyed.
			const FConcurrencyObjectID ConcurrencyObjectID = ConcurrencyGroup->GetObjectID();

			// Remove the object from the map.
			ConcurrencyGroups.Remove(ConcurrencyGroupID);

			// Remove from global group map if present.
			ConcurrencyMap.Remove(ConcurrencyObjectID);

			// Remove from sound object map if present.
			if (USoundBase* Sound = ActiveSound.GetSound())
			{
				const FSoundOwnerObjectID ObjectID = Sound->GetUniqueID();
				SoundObjectToConcurrencyGroup.Remove(ObjectID);
			}

			// Remove from owner map if present.
			const uint32 OwnerID = ActiveSound.GetOwnerID();
			if (FOwnerConcurrencyMapEntry* OwnerEntry = OwnerConcurrencyMap.Find(OwnerID))
			{
				if (OwnerEntry->ConcurrencyObjectToConcurrencyGroup.Remove(ConcurrencyObjectID))
				{
					if (OwnerEntry->ConcurrencyObjectToConcurrencyGroup.Num() == 0)
					{
						OwnerConcurrencyMap.Remove(OwnerID);
					}
				}
			}

			// Remove from owner per sound map if present.
			if (FSoundInstanceEntry* InstanceEntry = OwnerPerSoundConcurrencyMap.Find(OwnerID))
			{
				if (USoundBase* Sound = ActiveSound.GetSound())
				{
					if (InstanceEntry->SoundInstanceToConcurrencyGroup.Remove(Sound->GetUniqueID()))
					{
						if (InstanceEntry->SoundInstanceToConcurrencyGroup.Num() == 0)
						{
							OwnerPerSoundConcurrencyMap.Remove(OwnerID);
						}
					}
				}
			}

			delete ConcurrencyGroup;
		}
	}

	ActiveSound.ConcurrencyGroupData.Reset();
}

void FSoundConcurrencyManager::StopDueToVoiceStealing(FActiveSound& ActiveSound)
{
	check(ActiveSound.AudioDevice);

	float FadeOutDuration = 0.0f;
	bool bRequiresConcurrencyFade = ActiveSound.GetConcurrencyFadeDuration(FadeOutDuration);
	if (bRequiresConcurrencyFade)
	{
		ActiveSound.AudioDevice->UnlinkActiveSoundFromComponent(ActiveSound);
	}
	else
	{
#if UE_AUDIO_PROFILERTRACE_ENABLED
		Audio::Trace::EventLog::SendActiveSoundEvent(ActiveSound, Audio::Trace::EventLog::ID::StopRequestConcurrency);
#endif // UE_AUDIO_PROFILERTRACE_ENABLED

		ActiveSound.AudioDevice->AddSoundToStop(&ActiveSound);
	}

	const bool bDoRangeCheck = false;
	FAudioVirtualLoop VirtualLoop;
	if (FAudioVirtualLoop::Virtualize(ActiveSound, bDoRangeCheck, VirtualLoop))
	{
		ActiveSound.ClearAudioComponent();
		if (USoundBase* Sound = ActiveSound.GetSound())
		{
			UE_LOGF(LogAudioConcurrency, Verbose, "Playing ActiveSound %ls Virtualizing: Sound's voice stollen due to concurrency group maximum met.", *Sound->GetName());
		}
		ActiveSound.AudioDevice->AddVirtualLoop(VirtualLoop);
	}

	// Apply concurrency fade after potentially virtualizing to avoid transferring undesired new concurrency fade state
	if (bRequiresConcurrencyFade)
	{
		ActiveSound.FadeOut = FActiveSound::EFadeOut::Concurrency;
		ActiveSound.ComponentVolumeFader.StartFade(0.0f, FadeOutDuration, Audio::EFaderCurve::Logarithmic);
		ActiveSound.ComponentVolumeFader.SetActiveDuration(FadeOutDuration);
	}
}

void FSoundConcurrencyManager::UpdateVolumeScaleGenerations()
{
	check(IsInAudioThread());

	for (auto& ConcurrencyGroupEntry : ConcurrencyGroups)
	{
		ConcurrencyGroupEntry.Value->UpdateGeneration();
	}
}

void FSoundConcurrencyManager::UpdateSoundsToCull()
{
	check(IsInAudioThread());

	for (auto& ConcurrenyGroupEntry : ConcurrencyGroups)
	{
		ConcurrenyGroupEntry.Value->CullSoundsDueToMaxConcurrency();
	}
}

