// Copyright Epic Games, Inc. All Rights Reserved.

#include "Sound/SoundCue.h"
#include "EdGraph/EdGraph.h"
#include "Engine/Engine.h"
#include "Misc/CoreDelegates.h"
#include "EdGraph/EdGraphSchema.h"
#include "UObject/UObjectIterator.h"
#include "EngineUtils.h"
#include "IAudioParameterTransmitter.h"
#include "Sound/SoundNodeAttenuation.h"
#include "Sound/SoundNodeQualityLevel.h"
#include "Sound/SoundNodeRandom.h"
#include "Sound/SoundNodeSoundClass.h"
#include "Sound/SoundNodeWavePlayer.h"
#include "GameFramework/GameUserSettings.h"
#include "AudioCompressionSettingsUtils.h"
#include "AudioDevice.h"
#if WITH_EDITOR
#endif // WITH_EDITOR

#include "Interfaces/ITargetPlatform.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SoundCue)

/*-----------------------------------------------------------------------------
	USoundCue implementation.
-----------------------------------------------------------------------------*/

int32 USoundCue::CachedQualityLevel = -1;

#if WITH_EDITOR
TSharedPtr<ISoundCueAudioEditor> USoundCue::SoundCueAudioEditor = nullptr;
#endif // WITH_EDITOR

USoundCue::USoundCue(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	VolumeMultiplier = 0.75f;
	PitchMultiplier = 1.0f;
	SubtitlePriority = DEFAULT_SUBTITLE_PRIORITY;
	CookedQualityIndex = INDEX_NONE;
	bIsRetainingAudio = false;
}

#if WITH_EDITOR
void USoundCue::PostInitProperties()
{
	Super::PostInitProperties();
	if (!HasAnyFlags(RF_ClassDefaultObject | RF_NeedLoad))
	{
		CreateGraph();
	}

	CacheAggregateValues();
}

void USoundCue::AddReferencedObjects(UObject* InThis, FReferenceCollector& Collector)
{
	USoundCue* This = CastChecked<USoundCue>(InThis);

	Collector.AddReferencedObject(This->SoundCueGraph, This);

	Super::AddReferencedObjects(InThis, Collector);
}
#endif // WITH_EDITOR

void USoundCue::CacheAggregateValues()
{
	if (FirstNode)
	{
		FirstNode->ConditionalPostLoad();

		if (GIsEditor)
		{
			const float NewDuration = FirstNode->GetDuration();
#if WITH_EDITOR
			if (!FMath::IsNearlyEqual(NewDuration, Duration) && FMath::IsNearlyZero(Duration))
			{
				UE_LOGF(LogAudio, Display, "Cached duration for Sound Cue %ls was zero and has changed. Consider manually Re-saving the asset", *GetFullName());
			}
#endif // #if WITH_EDITOR
			Duration = NewDuration;
		}

		MaxDistance = ComputeMaxDistance();
		bHasDelayNode = FirstNode->HasDelayNode();
		bHasConcatenatorNode = FirstNode->HasConcatenatorNode();
		bHasPlayWhenSilent = FirstNode->IsPlayWhenSilent();
		bHasAttenuationNode = FirstNode->HasAttenuationNode();
	}
}

void USoundCue::PrimeSoundCue()
{
	if (FirstNode != nullptr)
	{
		FirstNode->PrimeChildWavePlayers(true);
	}
}

void USoundCue::RetainSoundCue()
{
	if (FirstNode)
	{
		FirstNode->RetainChildWavePlayers(true);
	}
	bIsRetainingAudio = true;
}

void USoundCue::ReleaseRetainedAudio()
{
	if (FirstNode)
	{
		FirstNode->ReleaseRetainerOnChildWavePlayers(true);
	}

	bIsRetainingAudio = false;
}

void USoundCue::CacheLoadingBehavior(ESoundWaveLoadingBehavior InBehavior)
{
	if (FirstNode)
	{
		FirstNode->OverrideLoadingBehaviorOnChildWaves(true, InBehavior);
	}
}

void USoundCue::Serialize(FStructuredArchive::FRecord Record)
{
	FArchive& UnderlyingArchive = Record.GetUnderlyingArchive();

	// Always force the duration to be updated when we are saving or cooking
	if (UnderlyingArchive.IsSaving() || UnderlyingArchive.IsCooking())
	{
		Duration = (FirstNode ? FirstNode->GetDuration() : 0.f);
		CacheAggregateValues();
	}

#if WITH_EDITOR
	// If we are cooking, record our cooked quality before serialize and then undo it.
	if (UnderlyingArchive.IsCooking() && UnderlyingArchive.IsSaving() && UnderlyingArchive.CookingTarget())
	{		
		if (const FPlatformAudioCookOverrides* Overrides = FPlatformCompressionUtilities::GetCookOverrides(*UnderlyingArchive.CookingTarget()->IniPlatformName()))
		{
			FScopeLock Lock(&EditorOnlyCs);
			CookedQualityIndex = Overrides->SoundCueCookQualityIndex;
			Super::Serialize(Record);
			CookedQualityIndex = INDEX_NONE;
		}
	}
	else
#endif //WITH_EDITOR
	{
		Super::Serialize(Record);
	}

	if (UnderlyingArchive.UEVer() >= VER_UE4_COOKED_ASSETS_IN_EDITOR_SUPPORT)
	{
		FStripDataFlags StripFlags(Record.EnterField(TEXT("SoundCueStripFlags")));
#if WITH_EDITORONLY_DATA
		if (!StripFlags.IsEditorDataStripped())
		{
			Record << SA_VALUE(TEXT("SoundCueGraph"), SoundCueGraph);
		}
#endif // WITH_EDITORONLY_DATA
	}
#if WITH_EDITOR
	else
	{
		Record << SA_VALUE(TEXT("SoundCueGraph"), SoundCueGraph);
	}
#endif // WITH_EDITOR
}

void USoundCue::PostLoad()
{
	Super::PostLoad();

	// Game doesn't care if there are NULL graph nodes
#if WITH_EDITOR
	if (GIsEditor && !GetOutermost()->HasAnyPackageFlags(PKG_FilterEditorOnly))
	{
		// we should have a soundcuegraph unless we are contained in a package which is missing editor only data
		if (ensure(SoundCueGraph))
		{
			USoundCue::GetSoundCueAudioEditor()->RemoveNullNodes(this);
		}

		// Always load all sound waves in the editor
		for (USoundNode* SoundNode : AllNodes)
		{
			if (USoundNodeAssetReferencer* AssetReferencerNode = Cast<USoundNodeAssetReferencer>(SoundNode))
			{
				AssetReferencerNode->LoadAsset();
			}
		}
	}
	else
#endif // WITH_EDITOR

	// Warn if the Quality index is set to something that we can't support.
	UE_CLOGF(USoundCue::GetCachedQualityLevel() != CookedQualityIndex && CookedQualityIndex != INDEX_NONE, LogAudio, Verbose,
		"'%ls' is igoring Quality Setting '%ls'(%d) as it was cooked with '%ls'(%d)",
		*GetFullNameSafe(this),
		*GetDefault<UAudioSettings>()->FindQualityNameByIndex(USoundCue::GetCachedQualityLevel()),
		USoundCue::GetCachedQualityLevel(),
		*GetDefault<UAudioSettings>()->FindQualityNameByIndex(CookedQualityIndex),
		CookedQualityIndex
	);

	if (GEngine && *GEngine->GameUserSettingsClass)
	{
		EvaluateNodes(false);
	}
	else
	{
		OnPostEngineInitHandle = FCoreDelegates::GetOnPostEngineInit().AddUObject(this, &USoundCue::OnPostEngineInit);
	}

	CacheAggregateValues();
	
	ESoundWaveLoadingBehavior SoundClassLoadingBehavior = ESoundWaveLoadingBehavior::Inherited;

	USoundClass* CurrentSoundClass = GetSoundClass();

	// Recurse through this sound class's parents until we find an override.
	while (SoundClassLoadingBehavior == ESoundWaveLoadingBehavior::Inherited && CurrentSoundClass != nullptr)
	{
		SoundClassLoadingBehavior = CurrentSoundClass->Properties.LoadingBehavior;
		CurrentSoundClass = CurrentSoundClass->ParentClass;
	}

	// RETAIN behavior gets demoted to PRIME in editor because in editor so many sound waves are technically "loaded" at a time.
	// If the Cache were not willing to evict them, that would cause issues.
	if (!GIsEditor && SoundClassLoadingBehavior == ESoundWaveLoadingBehavior::RetainOnLoad)
	{
		RetainSoundCue();
	}
	else if (bPrimeOnLoad
		|| (SoundClassLoadingBehavior == ESoundWaveLoadingBehavior::PrimeOnLoad)
		|| (GIsEditor && (SoundClassLoadingBehavior == ESoundWaveLoadingBehavior::RetainOnLoad))
		)
	{
		// In editor, we call this for RetainOnLoad behavior as well
		PrimeSoundCue();
	}
	else if (SoundClassLoadingBehavior == ESoundWaveLoadingBehavior::LoadOnDemand)
	{
		// update the soundWaves with the loading behavior they inherited
		CacheLoadingBehavior(SoundClassLoadingBehavior);
 	}
}

bool USoundCue::CanBeClusterRoot() const
{
	return true;
}

bool USoundCue::CanBeInCluster() const
{
	return true;
}

void USoundCue::OnPostEngineInit()
{
	FCoreDelegates::GetOnPostEngineInit().Remove(OnPostEngineInitHandle);
	OnPostEngineInitHandle.Reset();

	EvaluateNodes(true);
}

void USoundCue::EvaluateNodes(bool bAddToRoot)
{
	CacheQualityLevel();


	TFunction<void(USoundNode*)> EvaluateNodes_Internal = [&](USoundNode* SoundNode)
	{
		if (SoundNode == nullptr)
		{
			return;
		}

		if (USoundNodeAssetReferencer* AssetReferencerNode = Cast<USoundNodeAssetReferencer>(SoundNode))
		{
			AssetReferencerNode->ConditionalPostLoad();
			AssetReferencerNode->LoadAsset(bAddToRoot);
		}
		else if (USoundNodeQualityLevel* QualityLevelNode = Cast<USoundNodeQualityLevel>(SoundNode))
		{
			if (QualityLevelNode->ChildNodes.IsValidIndex(CachedQualityLevel))
			{
				EvaluateNodes_Internal(QualityLevelNode->ChildNodes[CachedQualityLevel]);
			}
		}
		else
		{
			for (USoundNode* ChildNode : SoundNode->ChildNodes)
			{
				EvaluateNodes_Internal(ChildNode);
			}
		}
	};

	TFunction<void(USoundNode*)> LoadProceduralAssets = [&](USoundNode* SoundNode)
	{
		if (SoundNode == nullptr)
		{
			return;
		}

		if (USoundNodeAssetReferencer* AssetReferencerNode = Cast<USoundNodeAssetReferencer>(SoundNode))
		{
			if (AssetReferencerNode->ContainsProceduralSoundReference())
			{
				AssetReferencerNode->ConditionalPostLoad();
				AssetReferencerNode->LoadAsset(bAddToRoot);
			}
		}
		else if (USoundNodeQualityLevel* QualityLevelNode = Cast<USoundNodeQualityLevel>(SoundNode))
		{
			QualityLevelNode->LoadChildWavePlayers(bAddToRoot, /*bRecurse=*/true);
		}
		else
		{
			for (USoundNode* ChildNode : SoundNode->ChildNodes)
			{
				LoadProceduralAssets(ChildNode);
			}
		}
	};

	// Only Evaluate nodes if we haven't been cooked, as cooked builds will hard-ref all SoundAssetReferences.	
	UE_CLOGF(CookedQualityIndex == INDEX_NONE, LogAudio, Verbose, "'%ls', DOING EvaluateNodes as we are *NOT* cooked", *GetName());
	UE_CLOGF(CookedQualityIndex != INDEX_NONE, LogAudio, Verbose, "'%ls', SKIPPING EvaluateNodes as we *ARE* cooked", *GetName());

	if (CookedQualityIndex == INDEX_NONE)
	{		
		EvaluateNodes_Internal(FirstNode);
	}
	else
	{
		// We need to load procedural assets (MetaSounds) to initialize their resources
		// before playing (which EvaluateNodes_Internal does in the other case)
		LoadProceduralAssets(FirstNode);
	}
}

void USoundCue::CacheQualityLevel()
{
	if (CachedQualityLevel == -1)
	{
		// Use per-platform quality index override if one exists, otherwise use the quality level from the game settings.
		CachedQualityLevel = FPlatformCompressionUtilities::GetQualityIndexOverrideForCurrentPlatform();
		if (CachedQualityLevel < 0)
		{
			CachedQualityLevel = GEngine->GetGameUserSettings()->GetAudioQualityLevel();
		}
	}
}

float USoundCue::ComputeMaxDistance() const
{
	float OutMaxDistance = 0.0f;
	if (const FSoundAttenuationSettings* Settings = GetAttenuationSettingsToApply())
	{
		if (!Settings->bAttenuate)
		{
			return FAudioDevice::GetMaxWorldDistance();
		}

		OutMaxDistance = FMath::Max(OutMaxDistance, Settings->GetMaxDimension());
	}

	if (FirstNode)
	{
		OutMaxDistance = FMath::Max(OutMaxDistance, FirstNode->GetMaxDistance());
	}

	if (OutMaxDistance > UE_KINDA_SMALL_NUMBER)
	{
		return OutMaxDistance;
	}

	// If no sound cue nodes has overridden the max distance, check the base attenuation
	return USoundBase::GetMaxDistance();
}


#if WITH_EDITOR

void USoundCue::RecursivelySetExcludeBranchCulling(USoundNode* CurrentNode)
{
	if (CurrentNode)
	{
		USoundNodeRandom* RandomNode = Cast<USoundNodeRandom>(CurrentNode);
		if (RandomNode)
		{
			RandomNode->bSoundCueExcludedFromBranchCulling = bExcludeFromRandomNodeBranchCulling;
			RandomNode->MarkPackageDirty();
		}
		for (USoundNode* ChildNode : CurrentNode->ChildNodes)
		{
			RecursivelySetExcludeBranchCulling(ChildNode);
		}
	}
}

void USoundCue::PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	if (PropertyChangedEvent.Property)
	{
		for (TObjectIterator<UAudioComponent> It; It; ++It)
		{
			if (It->Sound == this && It->IsActive())
			{
				// Allow attenuation overrides not update without stopping
				if (PropertyChangedEvent.MemberProperty->GetFName() == GET_MEMBER_NAME_STRING_CHECKED(USoundCue, AttenuationOverrides))
				{
					It->SetAttenuationOverrides(AttenuationOverrides);
				}
				else if (PropertyChangedEvent.MemberProperty->GetFName() == GET_MEMBER_NAME_STRING_CHECKED(USoundCue, bOverrideAttenuation))
				{
					It->SetAttenuationOverrides(AttenuationOverrides);
					It->SetOverrideAttenuation(bOverrideAttenuation);
				}
				else
				{
					It->Stop();
					It->Play();
				}

			}
		}

		// Propagate branch exclusion to child nodes which care (sound node random)
		RecursivelySetExcludeBranchCulling(FirstNode);
	}

	CacheAggregateValues();
}
#endif // WITH_EDITOR

void USoundCue::RecursiveFindAttenuation(USoundNode* Node, TArray<USoundNodeAttenuation*> &OutNodes)
{
	RecursiveFindNode<USoundNodeAttenuation>( Node, OutNodes );
}

void USoundCue::RecursiveFindAttenuation(const USoundNode* Node, TArray<const USoundNodeAttenuation*>& OutNodes) const
{
	RecursiveFindNode<USoundNodeAttenuation>(Node, OutNodes);
}

void USoundCue::RecursiveFindAllNodes( USoundNode* Node, TArray<class USoundNode*> &OutNodes )
{
	if( Node )
	{
		OutNodes.AddUnique( Node );

		// Recurse.
		const int32 MaxChildNodes = Node->GetMaxChildNodes();
		for( int32 ChildIndex = 0 ; ChildIndex < Node->ChildNodes.Num() && ChildIndex < MaxChildNodes ; ++ChildIndex )
		{
			RecursiveFindAllNodes( Node->ChildNodes[ ChildIndex ], OutNodes );
		}
	}
}

bool USoundCue::RecursiveFindPathToNode(USoundNode* CurrentNode, const UPTRINT CurrentHash, const UPTRINT NodeHashToFind, TArray<USoundNode*>& OutPath) const
{
	OutPath.Push(CurrentNode);
	if (CurrentHash == NodeHashToFind)
	{
		return true;
	}

	for (int32 ChildIndex = 0; ChildIndex < CurrentNode->ChildNodes.Num(); ++ChildIndex)
	{
		USoundNode* ChildNode = CurrentNode->ChildNodes[ChildIndex];
		if (ChildNode)
		{
			if (RecursiveFindPathToNode(ChildNode, USoundNode::GetNodeWaveInstanceHash(CurrentHash, ChildNode, ChildIndex), NodeHashToFind, OutPath))
			{
				return true;
			}
		}
	}

	OutPath.Pop();
	return false;
}

bool USoundCue::FindPathToNode(const UPTRINT NodeHashToFind, TArray<USoundNode*>& OutPath) const
{
	return RecursiveFindPathToNode(FirstNode, (UPTRINT)FirstNode, NodeHashToFind, OutPath);
}

void USoundCue::StaticAudioQualityChanged(int32 NewQualityLevel)
{
	if (CachedQualityLevel != NewQualityLevel)
	{
		FAudioCommandFence AudioFence;
		AudioFence.BeginFence();
		AudioFence.Wait();

		CachedQualityLevel = NewQualityLevel;

		if (GEngine)
		{
			for (TObjectIterator<USoundCue> SoundCueIt; SoundCueIt; ++SoundCueIt)
			{
				SoundCueIt->AudioQualityChanged();
			}
		}
		else
		{
			// PostLoad should have set up the delegate to fire EvaluateNodes once GEngine is initialized
		}
	}
}

void USoundCue::AudioQualityChanged()
{
	// First clear any references to assets that were loaded in the old child nodes
	TArray<USoundNode*> NodesToClearReferences;
	NodesToClearReferences.Push(FirstNode);

	while (NodesToClearReferences.Num() > 0)
	{
		if (USoundNode* SoundNode = NodesToClearReferences.Pop(EAllowShrinking::No))
		{
			if (USoundNodeAssetReferencer* AssetReferencerNode = Cast<USoundNodeAssetReferencer>(SoundNode))
			{
				AssetReferencerNode->ClearAssetReferences();
			}
			else
			{
				NodesToClearReferences.Append(SoundNode->ChildNodes);
			}
		}
	}

	// Now re-evaluate the nodes to reassign the references to any objects that are still legitimately
	// referenced and load any new assets that are now referenced that were not previously
	EvaluateNodes(false);
}

void USoundCue::BeginDestroy()
{
	Super::BeginDestroy();
}

FString USoundCue::GetDesc()
{
	FString Description;

	// Display duration
	const float CueDuration = GetDuration();
	if( CueDuration < INDEFINITELY_LOOPING_DURATION )
	{
		Description = FString::Printf( TEXT( "%3.2fs" ), CueDuration );
	}
	else
	{
		Description = TEXT( "Forever" );
	}

	// Display group
	Description += TEXT( " [" );
	Description += *GetSoundClass()->GetName();
	Description += TEXT( "]" );

	return Description;
}

int32 USoundCue::GetResourceSizeForFormat(FName Format)
{
	TArray<USoundNodeWavePlayer*> WavePlayers;
	RecursiveFindNode<USoundNodeWavePlayer>(FirstNode, WavePlayers);

	int32 ResourceSize = 0;
	for (int32 WaveIndex = 0; WaveIndex < WavePlayers.Num(); ++WaveIndex)
	{
		USoundWave* SoundWave = WavePlayers[WaveIndex]->GetSoundWave();
		if (SoundWave)
		{
			ResourceSize += SoundWave->GetResourceSizeForFormat(Format);
		}
	}

	return ResourceSize;
}

float USoundCue::GetMaxDistance() const
{
	// Always recalc the max distance when in the editor as it could change
	// from a referenced attenuation asset being updated without this cue
	// asset re-caching the aggregate 'MaxDistance' value
	return GIsEditor ? ComputeMaxDistance() : MaxDistance;
}

float USoundCue::GetDuration() const
{
	// Always recalc the duration when in the editor as it could change
	if (GIsEditor || (Duration < UE_SMALL_NUMBER) || HasDelayNode())
	{
		// This needs to be cached here vs an earlier point due to the need to parse sound cues and load order issues.
		// Alternative is to make getters not const, this is preferable. 
		USoundCue* ThisSoundCue = const_cast<USoundCue*>(this);
		ThisSoundCue->CacheAggregateValues();
	}

	return Duration;
}


bool USoundCue::ShouldApplyInteriorVolumes()
{
	// Only evaluate the sound class graph if we've not cached the result or if we're in editor
	if (GIsEditor || !bShouldApplyInteriorVolumesCached)
	{
		// After this, we'll have cached the value
		bShouldApplyInteriorVolumesCached = true;

		bShouldApplyInteriorVolumes = Super::ShouldApplyInteriorVolumes();

		// Only need to evaluate the sound cue graph if our super doesn't have apply interior volumes enabled
		if (!bShouldApplyInteriorVolumes)
		{
			TArray<UObject*> Children;
			GetObjectsWithOuter(this, Children);

			for (UObject* Child : Children)
			{
				if (USoundNodeSoundClass* SoundClassNode = Cast<USoundNodeSoundClass>(Child))
				{
					if (SoundClassNode->SoundClassOverride && SoundClassNode->SoundClassOverride->Properties.bApplyAmbientVolumes)
					{
						bShouldApplyInteriorVolumes = true;
						break;
					}
				}
			}
		}
	}

	return bShouldApplyInteriorVolumes;
}

bool USoundCue::IsPlayable() const
{
	return FirstNode != nullptr;
}

bool USoundCue::IsPlayWhenSilent() const
{
	if (VirtualizationMode == EVirtualizationMode::PlayWhenSilent)
	{
		return true;
	}

	return bHasPlayWhenSilent;
}

void USoundCue::Parse(FAudioDevice* AudioDevice, const UPTRINT NodeWaveInstanceHash, FActiveSound& ActiveSound, const FSoundParseParameters& ParseParams, TArray<FWaveInstance*>& WaveInstances)
{
	using namespace Audio;

	if (FirstNode)
	{
		FirstNode->ParseNodes(AudioDevice, (UPTRINT)FirstNode, ActiveSound, ParseParams, WaveInstances);
	}

	if (FSoundCueParameterTransmitter* Transmitter = static_cast<FSoundCueParameterTransmitter*>(ActiveSound.GetTransmitter()))
	{
		TArray<FAudioParameter> ParameterUpdates = Transmitter->ReleaseAccumulatedParameterUpdates();
		if (ParameterUpdates.IsEmpty())
		{
			return;
		}

		if (WaveInstances.IsEmpty())
		{
			return;
		}

		auto UpdateChildParameters = [](IParameterTransmitter& InChildTransmitter, USoundBase* InSound, TArray<FAudioParameter> InParameters)
		{
			if (InSound)
			{
				InSound->InitParameters(InParameters);
			}

			if (!InParameters.IsEmpty())
			{
				InChildTransmitter.SetParameters(MoveTemp(InParameters));
			}
		};

		const FWaveInstance* LastInstance = WaveInstances.Last();
		for (const FWaveInstance* Instance : WaveInstances)
		{
			if (!Instance)
			{
				continue;
			}
			
PRAGMA_DISABLE_DEPRECATION_WARNINGS	
			if (TSharedPtr<Audio::IParameterTransmitter>* ChildTransmitterPtr = Transmitter->Transmitters.Find(Instance->WaveInstanceHash))
PRAGMA_ENABLE_DEPRECATION_WARNINGS
			{
				TSharedPtr<Audio::IParameterTransmitter>& ChildTransmitter = *ChildTransmitterPtr;
				
				if (ChildTransmitterPtr->IsValid())
				{
					if (Instance == LastInstance)
					{
						// Avoid extra copy on last wave instance by moving parameters. 
						// This also avoids a copy on simple case where there is only one wave instance. 
						UpdateChildParameters(*ChildTransmitterPtr->Get(), Instance->WaveData, MoveTemp(ParameterUpdates));
					}
					else
					{
						UpdateChildParameters(*ChildTransmitterPtr->Get(), Instance->WaveData, ParameterUpdates);
					}
				}
			}
		}
	}
}

float USoundCue::GetVolumeMultiplier()
{
	return VolumeMultiplier;
}

float USoundCue::GetPitchMultiplier()
{
	return PitchMultiplier;
}

const FSoundAttenuationSettings* USoundCue::GetAttenuationSettingsToApply() const
{
	if (bOverrideAttenuation)
	{
		return &AttenuationOverrides;
	}
	return Super::GetAttenuationSettingsToApply();
}

float USoundCue::EvaluateMaxAttenuation(const FTransform& Origin, FVector Location, float DistanceScale /*= 1.f*/) const
{
	if (!bHasAttenuationNode)
	{
		if (const FSoundAttenuationSettings* Att = GetAttenuationSettingsToApply())
		{
			return Att->Evaluate(Origin, Location, DistanceScale);
		}
		else
		{
			return 1.0f;
		}
	}

	// Otherwise let's traverse recursively through our attenuation nodes and tally up the highest eval to return
	TArray<const USoundNodeAttenuation*> Nodes;
	RecursiveFindAttenuation(FirstNode, Nodes);

	float MaxEval = 0.0f;
	for (const USoundNodeAttenuation* Node : Nodes)
	{
		if (Node == nullptr)
		{
			continue;
		}

		if (Node->bOverrideAttenuation)
		{
			MaxEval = FMath::Max(MaxEval, Node->AttenuationOverrides.Evaluate(Origin, Location, DistanceScale));
		}
		else if (Node->AttenuationSettings)
		{
			MaxEval = FMath::Max(MaxEval, Node->AttenuationSettings->Attenuation.Evaluate(Origin, Location, DistanceScale));
		}
		else
		{
			MaxEval = 1.0f;
		}
	}
	return MaxEval;
}

float USoundCue::GetSubtitlePriority() const
{
	return SubtitlePriority;
}

bool USoundCue::GetSoundWavesWithCookedAnalysisData(TArray<USoundWave*>& OutSoundWaves)
{
	// Check this sound cue's wave players to see if any of their soundwaves have cooked analysis data
	TArray<USoundNodeWavePlayer*> WavePlayers;
	RecursiveFindNode<USoundNodeWavePlayer>(FirstNode, WavePlayers);

	bool bHasAnalysisData = false;
	for (USoundNodeWavePlayer* Player : WavePlayers)
	{
		USoundWave* SoundWave = Player->GetSoundWave();
		if (SoundWave && SoundWave->GetSoundWavesWithCookedAnalysisData(OutSoundWaves))
		{
			bHasAnalysisData = true;
		}
	}
	return bHasAnalysisData;
}

bool USoundCue::HasCookedFFTData() const
{
	// Check this sound cue's wave players to see if any of their soundwaves have cooked analysis data
	TArray<const USoundNodeWavePlayer*> WavePlayers;
	RecursiveFindNode<USoundNodeWavePlayer>(FirstNode, WavePlayers);

	for (const USoundNodeWavePlayer* Player : WavePlayers)
	{
		const USoundWave* SoundWave = Player->GetSoundWave();
		if (SoundWave && SoundWave->HasCookedFFTData())
		{
			return true;
		}
	}
	return false;
}

bool USoundCue::HasCookedAmplitudeEnvelopeData() const
{
	// Check this sound cue's wave players to see if any of their soundwaves have cooked analysis data
	TArray<const USoundNodeWavePlayer*> WavePlayers;
	RecursiveFindNode<USoundNodeWavePlayer>(FirstNode, WavePlayers);

	for (const USoundNodeWavePlayer* Player : WavePlayers)
	{
		const USoundWave* SoundWave = Player->GetSoundWave();
		if (SoundWave && SoundWave->HasCookedAmplitudeEnvelopeData())
		{
			return true;
		}
	}
	return false;
}

TSharedPtr<Audio::IParameterTransmitter> USoundCue::CreateParameterTransmitter(Audio::FParameterTransmitterInitParams&& InParams) const
{
	return MakeShared<FSoundCueParameterTransmitter>(MoveTemp(InParams));
}

#if WITH_EDITOR
UEdGraph* USoundCue::GetGraph()
{
	return SoundCueGraph;
}

void USoundCue::CreateGraph()
{
	if (SoundCueGraph == nullptr)
	{
		SoundCueGraph = USoundCue::GetSoundCueAudioEditor()->CreateNewSoundCueGraph(this);
		SoundCueGraph->bAllowDeletion = false;

		// Give the schema a chance to fill out any required nodes (like the results node)
		const UEdGraphSchema* Schema = SoundCueGraph->GetSchema();
		Schema->CreateDefaultNodesForGraph(*SoundCueGraph);
	}
}

void USoundCue::ClearGraph()
{
	if (SoundCueGraph)
	{
		SoundCueGraph->Nodes.Empty();
		// Give the schema a chance to fill out any required nodes (like the results node)
		const UEdGraphSchema* Schema = SoundCueGraph->GetSchema();
		Schema->CreateDefaultNodesForGraph(*SoundCueGraph);
	}
}

void USoundCue::SetupSoundNode(USoundNode* InSoundNode, bool bSelectNewNode/* = true*/)
{
	// Create the graph node
	check(InSoundNode->GraphNode == NULL);

	USoundCue::GetSoundCueAudioEditor()->SetupSoundNode(SoundCueGraph, InSoundNode, bSelectNewNode);
}

void USoundCue::LinkGraphNodesFromSoundNodes()
{
	USoundCue::GetSoundCueAudioEditor()->LinkGraphNodesFromSoundNodes(this);
	CacheAggregateValues();
}

void USoundCue::CompileSoundNodesFromGraphNodes()
{
	USoundCue::GetSoundCueAudioEditor()->CompileSoundNodesFromGraphNodes(this);
}

void USoundCue::SetSoundCueAudioEditor(TSharedPtr<ISoundCueAudioEditor> InSoundCueAudioEditor)
{
	check(!SoundCueAudioEditor.IsValid());
	SoundCueAudioEditor = InSoundCueAudioEditor;
}

void USoundCue::ResetGraph()
{
	for (const USoundNode* Node : AllNodes)
	{
		SoundCueGraph->RemoveNode(Node->GraphNode);
	}

	AllNodes.Reset();
	FirstNode = nullptr;
}

/** Gets the sound cue graph editor implementation. */
TSharedPtr<ISoundCueAudioEditor> USoundCue::GetSoundCueAudioEditor()
{
	return SoundCueAudioEditor;
}
#endif // WITH_EDITOR
	   
PRAGMA_DISABLE_DEPRECATION_WARNINGS // Disabling deprecation warnings for construction/destruction of deprecated member variables. 
FSoundCueParameterTransmitter::FSoundCueParameterTransmitter(Audio::FParameterTransmitterInitParams&& InParams)
	: Audio::FParameterTransmitterBase(MoveTemp(InParams.DefaultParams))
{
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

PRAGMA_DISABLE_DEPRECATION_WARNINGS // Disabling deprecation warnings for construction/destruction of deprecated member variables.
FSoundCueParameterTransmitter::~FSoundCueParameterTransmitter()
{
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

TArray<const TObjectPtr<UObject>*> FSoundCueParameterTransmitter::GetReferencedObjects() const
{
	static bool bHasLoggedError = false;
	if (!bHasLoggedError)
	{
		UE_LOGF(LogAudio, Error, "Do not call FSoundCueParameterTransmitter::GetReferencedObjects() as it is not thread safe.");
		bHasLoggedError = true;
	}

	return {};
}

void FSoundCueParameterTransmitter::AddReferencedObjects(FReferenceCollector& InCollector)
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS // Disable deprecations because ParamsToSet is being moved to private
	AddReferencedObjectsFromParameters(InCollector, ParamsToSet);
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	Super::AddReferencedObjects(InCollector);
}

bool FSoundCueParameterTransmitter::SetParameters(TArray<FAudioParameter>&& InParameters)
{
	auto RemoveTriggerParameters = [&]()
	{
		for (int32 ParamIndex = InParameters.Num() - 1; ParamIndex >= 0; --ParamIndex)
		{
			// Triggers are transient and are not applied for virtualized sounds. 
			// If a cached value is desired, use SetBoolParameter
			// (see comment for IAudioParameterControllerInterface::SetTriggerParameter)
			FAudioParameter& Param = InParameters[ParamIndex];
			if (Param.ParamType == EAudioParameterType::Trigger)
			{
				InParameters.RemoveAtSwap(ParamIndex, 1, EAllowShrinking::No);
			}
		}
	};

	if (IsVirtualized())
	{
		RemoveTriggerParameters();

		TArray<FAudioParameter> TempParams = InParameters;
PRAGMA_DISABLE_DEPRECATION_WARNINGS // Disable deprecations because ParamsToSet is being moved to private
		FAudioParameter::Merge(MoveTemp(TempParams), ParamsToSet);
PRAGMA_ENABLE_DEPRECATION_WARNINGS

		return Super::SetParameters(MoveTemp(InParameters));
	}
	else
	{
		TArray<FAudioParameter> TempParams = InParameters;
PRAGMA_DISABLE_DEPRECATION_WARNINGS // Disable deprecations because ParamsToSet is being moved to private
		FAudioParameter::Merge(MoveTemp(TempParams), ParamsToSet);
PRAGMA_ENABLE_DEPRECATION_WARNINGS

		RemoveTriggerParameters();

		return Super::SetParameters(MoveTemp(InParameters));
	}
}

void FSoundCueParameterTransmitter::CreateChildTransmitter(UPTRINT InWaveInstanceHash, USoundBase& InSound, Audio::FParameterTransmitterInitParams InTransmitterInitParams)
{
	check(IsInAudioThread());

	// Override any default parameters with the existing parameters. 
	TArray<FAudioParameter> CurrentAudioParameters;
	CopyParameters(CurrentAudioParameters);
	FAudioParameter::Merge(MoveTemp(CurrentAudioParameters), InTransmitterInitParams.DefaultParams);

	InSound.InitParameters(InTransmitterInitParams.DefaultParams);
	// Create child transmitter and store if valid. 
	TSharedPtr<Audio::IParameterTransmitter> ChildTransmitter = InSound.CreateParameterTransmitter(MoveTemp(InTransmitterInitParams));

	if (ChildTransmitter.IsValid())
	{
PRAGMA_DISABLE_DEPRECATION_WARNINGS // Disable deprecations because transmitters is being moved to private
		Transmitters.Add(InWaveInstanceHash, ChildTransmitter.ToSharedRef());
PRAGMA_ENABLE_DEPRECATION_WARNINGS 
	}
}

void FSoundCueParameterTransmitter::RemoveChildTransmitter(UPTRINT InWaveInstanceHash)
{
	check(IsInAudioThread());
PRAGMA_DISABLE_DEPRECATION_WARNINGS // Disable deprecations because transmitters is being moved to private
	Transmitters.Remove(InWaveInstanceHash);
PRAGMA_ENABLE_DEPRECATION_WARNINGS 
}

TArray<FAudioParameter> FSoundCueParameterTransmitter::ReleaseAccumulatedParameterUpdates()
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS // Disable deprecations because ParamsToSet is being moved to private
	TArray<FAudioParameter> AccumulatedParameters(MoveTemp(ParamsToSet));
	ParamsToSet.Reset();
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	return MoveTemp(AccumulatedParameters);
}

