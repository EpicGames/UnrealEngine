// Copyright Epic Games, Inc. All Rights Reserved.

#include "Sound/SoundEffectBase.h"

#include "Sound/SoundEffectPreset.h"


FSoundEffectBase::FSoundEffectBase()
	: bChanged(true)
	, bIsRunning(false)
	, bIsActive(false)
{}

bool FSoundEffectBase::IsActive() const
{
	return bIsActive;
}

void FSoundEffectBase::SetEnabled(const bool bInIsEnabled)
{
	bIsActive = bInIsEnabled;
}

USoundEffectPreset* FSoundEffectBase::GetPreset()
{
	if (Preset.IsValid())
	{
		return Preset.Get();
	}

	return nullptr;
}

TWeakObjectPtr<USoundEffectPreset>& FSoundEffectBase::GetPresetPtr()
{
	return Preset;
}

void FSoundEffectBase::ClearPreset()
{
	Preset.Reset();
}

bool FSoundEffectBase::Update()
{
	PumpPendingMessages();

	if (bChanged && Preset.IsValid())
	{
		OnPresetChanged();
		bChanged = false;

		return true;
	}

	return false;
}

bool FSoundEffectBase::IsPreset(USoundEffectPreset* InPreset) const
{
	return Preset == InPreset;
}

void FSoundEffectBase::EffectCommand(TUniqueFunction<void()> Command)
{
	CommandQueue.Enqueue(MoveTemp(Command));
}

void FSoundEffectBase::PumpPendingMessages()
{
	// Pumps the command queue
	TUniqueFunction<void()> Command;
	while (CommandQueue.Dequeue(Command))
	{
		Command();
	}
}
