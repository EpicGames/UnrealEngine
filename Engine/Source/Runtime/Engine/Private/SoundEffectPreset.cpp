// Copyright Epic Games, Inc. All Rights Reserved.


#include "Sound/SoundEffectPreset.h"
#include "Sound/SoundEffectBase.h"
#include "Sound/SoundEffectSource.h"
#include "Engine/Engine.h"
#include "AudioDeviceManager.h"
#include "UObject/ObjectPtr.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SoundEffectPreset)

USoundEffectPreset::USoundEffectPreset(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, bInitialized(false)
{
}

void USoundEffectPreset::Update()
{
	FScopeLock ScopeLock(&InstancesMutationCriticalSection);
	for (int32 i = Instances.Num() - 1; i >= 0; --i)
	{
		TSoundEffectPtr EffectSharedPtr = Instances[i].Pin();
		if (!EffectSharedPtr.IsValid() || EffectSharedPtr->GetPreset() == nullptr)
		{
			Instances.RemoveAtSwap(i, 1);
		}
		else
		{
			RegisterInstance(*this, EffectSharedPtr);
		}
	}
}

void USoundEffectPreset::AddEffectInstance(TSoundEffectPtr& InEffectPtr)
{
	if (!bInitialized)
	{
		bInitialized = true;
		Init();

		// Call the optional virtual function which subclasses can implement if they need initialization
		OnInit();
	}

	FScopeLock ScopeLock(&InstancesMutationCriticalSection);
	Instances.AddUnique(TSoundEffectWeakPtr(InEffectPtr));
}

void USoundEffectPreset::AddReferencedEffects(FReferenceCollector& InCollector)
{
	FReferenceCollector* Collector = &InCollector;
	IterateEffects<FSoundEffectBase>([Collector](FSoundEffectBase& Instance)
	{
		if (auto& EffectPreset = Instance.GetPresetPtr(); EffectPreset.Get())
		{
			Collector->AddReferencedObject(EffectPreset);
		}
	});
}

void USoundEffectPreset::BeginDestroy()
{
	FScopeLock ScopeLock(&InstancesMutationCriticalSection);
	IterateEffects<FSoundEffectBase>([](FSoundEffectBase& Instance)
	{
		Instance.ClearPreset();
	});
	Instances.Reset();

	Super::BeginDestroy();
}

void USoundEffectPreset::RemoveEffectInstance(TSoundEffectPtr& InEffectPtr)
{
	FScopeLock ScopeLock(&InstancesMutationCriticalSection);
	Instances.RemoveSwap(TSoundEffectWeakPtr(InEffectPtr));
}

#if WITH_EDITOR
void USoundEffectPreset::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	// Copy the settings to the thread safe version
	Init();
	OnInit();
	Update();
}

void USoundEffectSourcePresetChain::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	if (GEngine)
	{
		if (FAudioDeviceManager* AudioDeviceManager = GEngine->GetAudioDeviceManager())
		{
			AudioDeviceManager->UpdateSourceEffectChain(GetUniqueID(), Chain, bPlayEffectChainTails);
		}
	}
}
#endif // WITH_EDITORONLY_DATA

void USoundEffectSourcePresetChain::AddReferencedEffects(FReferenceCollector& Collector)
{
	for (FSourceEffectChainEntry& SourceEffect : Chain)
	{
		if (SourceEffect.Preset)
		{
			SourceEffect.Preset->AddReferencedEffects(Collector);
		}
	}
}

int32 USoundEffectSourcePresetChain::GetSupportedChannelCount() const
{
	// Check the min number of channels the source effect chain supports
	// We don't want to instantiate the effect chain if it has an effect that doesn't support its channel count
	// E.g. we shouldn't instantiate a chain on a quad source if there is an effect in the chain that only supports stereo
	constexpr int32 InvalidChannelCount = AUDIO_MIXER_MAX_OUTPUT_CHANNELS + 1;
	int32 EffectChainSupportedChannels = InvalidChannelCount;

	for (const FSourceEffectChainEntry& SourceEffectChainEntry : Chain)
	{
		if (SourceEffectChainEntry.Preset)
		{
			EffectChainSupportedChannels = FMath::Min(SourceEffectChainEntry.Preset->GetMaxSupportedChannels(), EffectChainSupportedChannels);
		}
	}

	return (EffectChainSupportedChannels < InvalidChannelCount) ? EffectChainSupportedChannels : USoundEffectSourcePreset::DefaultSupportedChannels;
}

bool USoundEffectSourcePresetChain::SupportsChannelCount(const int32 InNumChannels) const
{
	return InNumChannels <= GetSupportedChannelCount();
}

void USoundEffectPreset::UnregisterInstance(TSoundEffectPtr InEffectPtr)
{
	if (ensure(IsInAudioThread() || IsInGameThread()))
	{
		if (InEffectPtr.IsValid())
		{
			if (USoundEffectPreset* Preset = InEffectPtr->GetPreset())
			{
				Preset->RemoveEffectInstance(InEffectPtr);
			}

			InEffectPtr->ClearPreset();
		}
	}
}

void USoundEffectPreset::RegisterInstance(USoundEffectPreset& InPreset, TSoundEffectPtr InEffectPtr)
{
	ensure(IsInAudioThread() || IsInGameThread());
	if (!InEffectPtr.IsValid())
	{
		return;
	}

	if (InEffectPtr->Preset.Get() != &InPreset)
	{
		UnregisterInstance(InEffectPtr);

		InEffectPtr->Preset = &InPreset;
		if (InEffectPtr->Preset.IsValid())
		{
			InPreset.AddEffectInstance(InEffectPtr);
		}
	}

	// Anytime notification occurs that the preset has been modified,
	// flag for update.
	InEffectPtr->bChanged = true;
}


