// Copyright Epic Games, Inc. All Rights Reserved.

#include "Sound/SoundEffectSubmix.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SoundEffectSubmix)

USoundEffectSubmixPreset::USoundEffectSubmixPreset(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

bool FSoundEffectSubmix::ProcessAudio(FSoundEffectSubmixInputData& InData, FSoundEffectSubmixOutputData& OutData)
{
	bIsRunning = true;
	InData.PresetData = nullptr;

	Update();

	// Only process the effect if the effect is active and the preset is valid
	if (bIsActive && Preset.IsValid())
	{
		OnProcessAudio(InData, OutData);
		return true;
	}

	return false;
}

