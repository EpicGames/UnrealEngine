// Copyright Epic Games, Inc. All Rights Reserved.

#include "Sound/SoundGenerator.h"
#include "AutoRTFM.h"
#include "HAL/LowLevelMemTracker.h"

ISoundGenerator::ISoundGenerator()
{
}

ISoundGenerator::~ISoundGenerator()
{
}

int32 ISoundGenerator::GetNextBuffer(float* OutAudio, const int32 NumSamples, bool bRequireNumberSamples)
{
	LLM_SCOPE(ELLMTag::AudioSynthesis);

	PumpPendingMessages();

	int32 NumSamplesToGenerate = NumSamples;
	if (!bRequireNumberSamples)
	{
		// Defer to the generator's desired block size
		NumSamplesToGenerate = FMath::Min(NumSamples, GetDesiredNumSamplesToRenderPerCallback());
	}

	int32 NumSamplesWritten = OnGenerateAudio(OutAudio, NumSamplesToGenerate);

	// If the generator didn't write all the required samples, zero out the rest
	if (bRequireNumberSamples && NumSamplesWritten != NumSamplesToGenerate)
	{
		check(NumSamplesWritten >= 0 && NumSamplesWritten < NumSamplesToGenerate);
		const int32 Remainder = NumSamplesToGenerate - NumSamplesWritten;
		FMemory::Memzero(OutAudio + NumSamplesWritten, Remainder * sizeof(float));
		NumSamplesWritten = NumSamplesToGenerate;
	}

	return NumSamplesWritten;
}

void ISoundGenerator::SynthCommand(TFunction<void()> Command)
{
	LLM_SCOPE(ELLMTag::AudioSynthesis);

	UE_AUTORTFM_ONCOMMIT(this, Command = MoveTemp(Command))
	{
		CommandQueue.Enqueue(MoveTemp(Command));
	};
}

void ISoundGenerator::PumpPendingMessages()
{
	TUniqueFunction<void()> Command;
	while (CommandQueue.Dequeue(Command))
	{
		Command();
	}
}


