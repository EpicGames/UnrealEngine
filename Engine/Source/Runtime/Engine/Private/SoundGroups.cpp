// Copyright Epic Games, Inc. All Rights Reserved.

#include "Sound/SoundGroups.h"
#include "Audio.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SoundGroups)

USoundGroups::USoundGroups(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

void USoundGroups::Initialize() const
{
	for (int32 ProfileIndex = 0; ProfileIndex < SoundGroupProfiles.Num(); ++ProfileIndex)
	{
		SoundGroupMap.Add(SoundGroupProfiles[ProfileIndex].SoundGroup, SoundGroupProfiles[ProfileIndex]);
	}

	if (!SoundGroupMap.Find(SOUNDGROUP_Default))
	{
		UE_LOGF(LogAudio, Warning, "Missing default SoundGroup profile. Creating default with no decompression.");

		SoundGroupMap.Add(SOUNDGROUP_Default, FSoundGroup());
	}

#if WITH_EDITOR
	UEnum* SoundGroupEnum = FindObjectChecked<UEnum>(NULL, TEXT("/Script/Engine.ESoundGroup"));

	for (const auto& It : SoundGroupMap)
	{
		if (It.Value.DisplayName.Len() > 0)
		{
			SoundGroupEnum->SetMetaData(TEXT("DisplayName"), *It.Value.DisplayName, It.Key);
			SoundGroupEnum->RemoveMetaData(TEXT("Hidden"), It.Key);
		}
		else
		{
			if (SoundGroupEnum->HasMetaData(TEXT("Hidden"), It.Key))
			{
				UE_LOGF(LogAudio, Warning, "Custom Game SoundGroup profile for %ls defined but no display name supplied.", *SoundGroupEnum->GetDisplayNameTextByValue(It.Key).ToString());
			}
			SoundGroupEnum->RemoveMetaData(TEXT("Hidden"), It.Key);
		}
	}
#endif
}

const FSoundGroup& USoundGroups::GetSoundGroup(const ESoundGroup SoundGroup) const
{
	// Initialize the settings if this gets called early enough to require it
	if (SoundGroupMap.Num() == 0)
	{
		Initialize();
	}

	const FSoundGroup* SG = SoundGroupMap.Find(SoundGroup);
	if (SG == NULL)
	{
		UEnum* SoundGroupEnum = FindObjectChecked<UEnum>(NULL, TEXT("/Script/Engine.ESoundGroup"));
		
		UE_LOGF(LogAudio, Warning, "Requested SoundGroup %ls does not have defined profile.  Using SOUNDGROUP_Default.", *SoundGroupEnum->GetDisplayNameTextByValue(SoundGroup).ToString());
		return SoundGroupMap.FindChecked(SOUNDGROUP_Default);
	}

	return *SG;
}
