// Copyright Epic Games, Inc. All Rights Reserved.


#include "Sound/SoundMix.h"
#include "Engine/Engine.h"
#include "AudioDeviceManager.h"
#include "Sound/SoundClass.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SoundMix)

#if WITH_EDITOR
#include "Framework/Notifications/NotificationManager.h"
#include "UObject/UnrealType.h"
#include "Widgets/Notifications/SNotificationList.h"
#endif

/*-----------------------------------------------------------------------------
	USoundMix implementation.
-----------------------------------------------------------------------------*/

USoundMix::USoundMix(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bApplyEQ = false;
	InitialDelay = 0.0f;
	Duration = -1.0f;
	FadeInTime = 0.2f;
	FadeOutTime = 0.2f;

#if WITH_EDITORONLY_DATA
	bChanged = false;
#endif
}

void USoundMix::BeginDestroy()
{
	if (!GExitPurge && GEngine)
	{
		FAudioDeviceManager* AudioDeviceManager = GEngine->GetAudioDeviceManager();
		if (AudioDeviceManager)
		{
			AudioDeviceManager->RemoveSoundMix(this);
		}
	}

	Super::BeginDestroy();
}

FString USoundMix::GetDesc( void )
{
	return( FString::Printf( TEXT( "Adjusters: %d" ), SoundClassEffects.Num() ) );
}

#if WITH_EDITOR
void USoundMix::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	if (PropertyChangedEvent.ChangeType != EPropertyChangeType::Interactive && PropertyChangedEvent.MemberProperty)
	{
		if (PropertyChangedEvent.MemberProperty->GetFName() == GET_MEMBER_NAME_CHECKED(USoundMix,SoundClassEffects))
		{
			TArray<USoundClass*> ProblemClasses;
			if (CausesPassiveDependencyLoop(ProblemClasses))
			{
				for (int32 Index = 0; Index < ProblemClasses.Num(); Index++)
				{
					FFormatNamedArguments Arguments;
					Arguments.Add(TEXT("SoundClass"), FText::FromString(ProblemClasses[Index]->GetName()));
					Arguments.Add(TEXT("SoundMix"), FText::FromString(GetName()));
					FNotificationInfo Info(FText::Format(NSLOCTEXT("Engine", "PassiveSoundMixLoop", "Passive dependency created by Sound Class'{SoundClass}' and Sound Mix'{SoundMix}' - results may be undesirable"), Arguments));
					Info.ExpireDuration = 10.0f;
					Info.Image = FCoreStyle::Get().GetBrush(TEXT("MessageLog.Warning"));
					FSlateNotificationManager::Get().AddNotification(Info);
				}
			}
		}
	}

#if WITH_EDITORONLY_DATA
	bChanged = true;
#endif
}

bool USoundMix::CausesPassiveDependencyLoop(TArray<USoundClass*>& ProblemClasses) const
{
	ProblemClasses.Empty();

	for (int32 Index = 0; Index < SoundClassEffects.Num(); Index++)
	{
		const FSoundClassAdjuster& CurrentAdjuster = SoundClassEffects[Index];

		// dependency loops are only a problem if volume is decreased,
		// which can potentially deactivate the Sound Mix
		if (CurrentAdjuster.SoundClassObject && CurrentAdjuster.VolumeAdjuster < 1.0f)
		{
			return CheckForDependencyLoop(CurrentAdjuster.SoundClassObject, ProblemClasses, CurrentAdjuster.bApplyToChildren);
		}
	}

	return false;
}

bool USoundMix::CheckForDependencyLoop(USoundClass* SoundClass, TArray<USoundClass*>& ProblemClasses, bool CheckChildren) const
{
	check(SoundClass);
	bool bFoundProblemClass = false;

	// check for circular references to passive sound mixes
	for (int32 Index = 0; Index < SoundClass->PassiveSoundMixModifiers.Num(); Index++)
	{
		const FPassiveSoundMixModifier& CurrentSoundMix = SoundClass->PassiveSoundMixModifiers[Index];

		// Ignore if volume threshold is between 0 and 10 (arbitrarily large upper value) as the Sound Mix will not be deactivated
		if (CurrentSoundMix.SoundMix == this && CurrentSoundMix.MinVolumeThreshold > 0.f && CurrentSoundMix.MaxVolumeThreshold < 10.f)
		{
			ProblemClasses.AddUnique(SoundClass);
			bFoundProblemClass = true;
		}
	}
	// check children if required
	if (CheckChildren)
	{
		for (int32 Index = 0; Index < SoundClass->ChildClasses.Num(); Index++)
		{
			if (SoundClass->ChildClasses[Index] && CheckForDependencyLoop(SoundClass->ChildClasses[Index], ProblemClasses, CheckChildren))
			{
				bFoundProblemClass = true;
			}
		}
	}
	return bFoundProblemClass;
}
#endif // WITH_EDITOR

