// Copyright Epic Games, Inc. All Rights Reserved.

#include "Sound/SoundModulationDestination.h"

#include "Algo/AnyOf.h"
#include "AudioDevice.h"
#include "AudioPluginUtilities.h"
#include "DSP/FloatArrayMath.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SoundModulationDestination)


FSoundModulationDefaultSettings::FSoundModulationDefaultSettings()
{
	VolumeModulationDestination.Value = 0.0f;
	PitchModulationDestination.Value = 0.0f;
	HighpassModulationDestination.Value = MIN_FILTER_FREQUENCY;
	LowpassModulationDestination.Value = MAX_FILTER_FREQUENCY;
}

#if WITH_EDITORONLY_DATA
void FSoundModulationDefaultSettings::VersionModulators()
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	VolumeModulationDestination.VersionModulators();
	PitchModulationDestination.VersionModulators();
	HighpassModulationDestination.VersionModulators();
	LowpassModulationDestination.VersionModulators();
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

void FSoundModulationDestinationSettings::VersionModulators()
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	if (Modulator)
	{
		Modulators.Add(Modulator);
		Modulator = nullptr;
	}
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}
#endif // WITH_EDITORONLY_DATA

namespace SoundModulationDestinationPrivate
{
	static bool IsPluginEnabled()
	{
		static bool bEnabled = AudioPluginUtilities::GetDesiredModulationPlugin() != nullptr;
		return bEnabled;
	}
}


FSoundModulationDefaultRoutingSettings::FSoundModulationDefaultRoutingSettings()
	: FSoundModulationDefaultSettings()
{
}

FSoundModulationDestinationSettings FSoundModulationDefaultSettings::GetSettingsForDestination(EModulationDestination Destination) const
{
	switch (Destination)
	{
	case EModulationDestination::Volume:
		return VolumeModulationDestination;
	case EModulationDestination::Pitch:
		return PitchModulationDestination;
	case EModulationDestination::Lowpass:
		return LowpassModulationDestination;
	case EModulationDestination::Highpass:
		return HighpassModulationDestination;
	default:
		break;
	}

	return FSoundModulationDestinationSettings();
}

EModulationRouting FSoundModulationDefaultRoutingSettings::GetRoutingForDestination(EModulationDestination Destination) const
{
	switch (Destination)
	{
		case EModulationDestination::Volume:
			return VolumeRouting;
		case EModulationDestination::Pitch:
			return PitchRouting;
		case EModulationDestination::Lowpass:
			return LowpassRouting;
		case EModulationDestination::Highpass:
			return HighpassRouting;
		default:
			break;
	}
	return EModulationRouting::Disable;
}


namespace Audio
{
	FModulationDestination::FModulationDestination(const FModulationDestination& InModulationDestination)
	{
		FScopeLock OtherLock(&InModulationDestination.DestinationData->HandleCritSection);
		*DestinationData = *InModulationDestination.DestinationData;
	}

	FModulationDestination::FModulationDestination(FModulationDestination&& InModulationDestination)
	{
		FScopeLock OtherLock(&InModulationDestination.DestinationData->HandleCritSection);
		*DestinationData = MoveTemp(*InModulationDestination.DestinationData);
	}

	FModulationDestination& FModulationDestination::operator=(const FModulationDestination& InModulationDestination)
	{
		*DestinationData = *(InModulationDestination.DestinationData);
		return *this;
	}

	FModulationDestination& FModulationDestination::operator=(FModulationDestination&& InModulationDestination)
	{
		*DestinationData = MoveTemp(*InModulationDestination.DestinationData);
		return *this;
	}

	void FModulationDestination::FModulationDestinationData::ResetHandles()
	{
		Audio::FModulationParameter ParameterCopy = Parameter;

		FScopeLock Lock(&HandleCritSection);
		Handles.Reset();
		Handles.Add(FModulatorHandle { MoveTemp(ParameterCopy) });
	}

	void FModulationDestination::Init(FDeviceId InDeviceId, bool bInIsBuffered, bool bInValueNormalized)
	{
		Init(InDeviceId, FName(), bInIsBuffered, bInValueNormalized);
	}

	void FModulationDestination::Init(FDeviceId InDeviceId, FName InParameterName, bool bInIsBuffered, bool bInValueNormalized)
	{
		DestinationData->DeviceId = InDeviceId;
		DestinationData->bIsBuffered = bInIsBuffered;
		DestinationData->bValueNormalized = bInValueNormalized;

		DestinationData->OutputBuffer.Reset();
		if (const FModulationParameter* Param = GetModulationParameterPtr(InParameterName))
		{
			DestinationData->Parameter = *Param;
		}
		else
		{
			UE_CLOGF(SoundModulationDestinationPrivate::IsPluginEnabled(),LogAudio, Warning, "Modulation Parameter %ls not found while Initializing Modulation Destination. Using default parameter instead.", *InParameterName.ToString());
			DestinationData->Parameter = Audio::GetDefaultModulationParameter();
		}

		DestinationData->ResetHandles();
	}

	bool FModulationDestination::IsActive() const
	{
		FScopeLock Lock(&DestinationData->HandleCritSection);
		return Algo::AnyOf(DestinationData->Handles, [](const FModulatorHandle& Handle) { return Handle.IsValid(); });
	}

	bool FModulationDestination::ProcessControl(float InValueUnitBase, int32 InNumSamples)
	{
		FModulationParameter& Parameter = DestinationData->Parameter;
		float& ValueTarget = DestinationData->ValueTarget;
		FAlignedFloatBuffer& OutputBuffer = DestinationData->OutputBuffer;
		
		DestinationData->bHasProcessed = true;
		float LastTarget = ValueTarget;


		float NewTargetNormalized = Parameter.DefaultValue;
		if (Parameter.bRequiresConversion)
		{
			Parameter.NormalizedFunction(NewTargetNormalized);
		}

		FScopeLock Lock(&DestinationData->HandleCritSection);
		{
			for (const FModulatorHandle& Handle : DestinationData->Handles)
			{
				if (Handle.IsValid())
				{
					float NewHandleValue = 1.0f;
					Handle.GetValue(NewHandleValue);
					Parameter.MixFunction(NewTargetNormalized, NewHandleValue);
				}
			}
		}

		// Convert base to linear space
		float InValueBaseNormalized = InValueUnitBase;
		if (Parameter.bRequiresConversion)
		{
			Parameter.NormalizedFunction(InValueBaseNormalized);
		}

		// Mix in base value
		Parameter.MixFunction(NewTargetNormalized, InValueBaseNormalized);
		ValueTarget = NewTargetNormalized;

		// Convert target to unit space if required
		if (Parameter.bRequiresConversion && !DestinationData->bValueNormalized)
		{
			Parameter.UnitFunction(ValueTarget);
		}

		if (DestinationData->bIsBuffered)
		{
			if (OutputBuffer.Num() != InNumSamples)
			{
				OutputBuffer.Reset();
				OutputBuffer.AddZeroed(InNumSamples);
			}
		}

		// Fade from last target to new if output buffer is active
		if (!OutputBuffer.IsEmpty())
		{
			if (OutputBuffer.Num() % AUDIO_NUM_FLOATS_PER_VECTOR_REGISTER == 0)
			{
				if (FMath::IsNearlyEqual(LastTarget, ValueTarget))
				{
					ArraySetToConstantInplace(OutputBuffer, ValueTarget);
				}
				else
				{
					ArraySetToConstantInplace(OutputBuffer, 1.0f);
					ArrayFade(OutputBuffer, LastTarget, ValueTarget);
				}
			}
			else
			{
				if (FMath::IsNearlyEqual(LastTarget, ValueTarget))
				{
					OutputBuffer.Init(ValueTarget, InNumSamples);
				}
				else
				{
					float SampleValue = LastTarget;
					const float DeltaValue = (ValueTarget - LastTarget) / OutputBuffer.Num();
					for (int32 i = 0; i < OutputBuffer.Num(); ++i)
					{
						OutputBuffer[i] = SampleValue;
						SampleValue += DeltaValue;
					}
				}
			}
		}

		return !FMath::IsNearlyEqual(LastTarget, ValueTarget);
	}

	void FModulationDestination::UpdateModulator(const USoundModulatorBase* InModulator)
	{
		SetModulators({ InModulator });
	}

	void FModulationDestination::UpdateModulators(const TSet<TObjectPtr<USoundModulatorBase>>& InModulators)
	{
		SetModulators(InModulators);
	}

	void FModulationDestination::UpdateModulators(const TSet<USoundModulatorBase*>& InModulators)
	{
		SetModulators(InModulators);
	}

	void FModulationDestination::UpdateModulators(const TSet<const USoundModulatorBase*>& InModulators)
	{
		SetModulators(InModulators);
	}

	void FModulationDestination::SetModulators(const TSet<TObjectPtr<USoundModulatorBase>>& InModulators)
	{
		TArray<TUniquePtr<Audio::IModulatorSettings>> ProxySettings;
		Algo::TransformIf(
			InModulators,
			ProxySettings,
			[](const USoundModulatorBase* Mod) { return Mod != nullptr; },
			[](const USoundModulatorBase* Mod) { return Mod->CreateProxySettings(); }
		);

		UpdateModulatorsInternal(MoveTemp(ProxySettings));
	}

	void FModulationDestination::SetModulators(const TSet<USoundModulatorBase*>& InModulators)
	{
		TArray<TUniquePtr<Audio::IModulatorSettings>> ProxySettings;
		Algo::TransformIf(
			InModulators,
			ProxySettings,
			[](const USoundModulatorBase* Mod) { return Mod != nullptr; },
			[](const USoundModulatorBase* Mod) { return Mod->CreateProxySettings(); }
		);

		UpdateModulatorsInternal(MoveTemp(ProxySettings));
	}

	void FModulationDestination::SetModulators(const TSet<const USoundModulatorBase*>& InModulators)
	{
		TArray<TUniquePtr<Audio::IModulatorSettings>> ProxySettings;
		Algo::TransformIf(
			InModulators,
			ProxySettings,
			[](const USoundModulatorBase* Mod) { return Mod != nullptr; },
			[](const USoundModulatorBase* Mod) { return Mod->CreateProxySettings(); }
		);

		UpdateModulatorsInternal(MoveTemp(ProxySettings));
	}

	void FModulationDestination::ClearModulators()
	{
		DestinationData->ResetHandles();
	}

	void FModulationDestination::FModulationDestinationData::SetHandles(TSet<FModulatorHandle>&& NewHandles)
	{
		FScopeLock Lock(&HandleCritSection);
		Handles = MoveTemp(NewHandles);
	}

	FModulationDestination::FModulationDestinationData& FModulationDestination::FModulationDestinationData::operator=(const FModulationDestinationData& InDestinationInfo)
	{
		DeviceId = InDestinationInfo.DeviceId;
		ValueTarget = InDestinationInfo.ValueTarget;
		bIsBuffered = InDestinationInfo.bIsBuffered;
		bValueNormalized = InDestinationInfo.bValueNormalized;
		OutputBuffer = InDestinationInfo.OutputBuffer;

		TSet<FModulatorHandle> NewHandles;
		{
			FScopeLock OtherLock(&(InDestinationInfo.HandleCritSection));
			NewHandles = InDestinationInfo.Handles;
		}

		{
			FScopeLock Lock(&HandleCritSection);
			Handles = MoveTemp(NewHandles);
		}

		Parameter = InDestinationInfo.Parameter;

		return *this;
	}

	FModulationDestination::FModulationDestinationData& FModulationDestination::FModulationDestinationData::operator=(FModulationDestinationData&& InDestinationInfo)
	{
		DeviceId = MoveTemp(InDestinationInfo.DeviceId);
		ValueTarget = MoveTemp(InDestinationInfo.ValueTarget);
		bIsBuffered = MoveTemp(InDestinationInfo.bIsBuffered);
		bValueNormalized = MoveTemp(InDestinationInfo.bValueNormalized);
		bHasProcessed = MoveTemp(InDestinationInfo.bHasProcessed);
		OutputBuffer = MoveTemp(InDestinationInfo.OutputBuffer);

		TSet<FModulatorHandle> NewHandles;
		{
			FScopeLock OtherLock(&InDestinationInfo.HandleCritSection);
			NewHandles = MoveTemp(InDestinationInfo.Handles);
		}
		{
			FScopeLock Lock(&HandleCritSection);
			Handles = MoveTemp(NewHandles);
		}

		Parameter = MoveTemp(InDestinationInfo.Parameter);

		return *this;
	}

	const FDeviceId& FModulationDestination::FModulationDestinationData::GetDeviceId() const
	{
		return DeviceId;
	}

	const FModulationParameter& FModulationDestination::FModulationDestinationData::GetParameter() const
	{
		return Parameter;
	}

	void FModulationDestination::UpdateModulatorsInternal(TArray<TUniquePtr<Audio::IModulatorSettings>>&& ProxySettings)
	{
		FAudioDeviceManager* AudioDeviceManager = FAudioDeviceManager::Get();
		if (!AudioDeviceManager)
		{
			return;
		}

		const FDeviceId DeviceId = DestinationData->GetDeviceId();
		FAudioDevice* AudioDevice = AudioDeviceManager->GetAudioDeviceRaw(DeviceId);
		if (!AudioDevice || !AudioDevice->IsModulationPluginEnabled() || !AudioDevice->ModulationInterface.IsValid())
		{
			return;
		}

		FAudioThread::RunCommandOnAudioThread(
		[
			DestinationDataPtr = TWeakPtr<FModulationDestinationData>(DestinationData),
			ModInterfacePtr = TWeakPtr<IAudioModulationManager>(AudioDevice->ModulationInterface),
			ModSettings = MoveTemp(ProxySettings)
		]() mutable
		{
			TSharedPtr<FModulationDestinationData> DestDataPtr = DestinationDataPtr.Pin();
			if (DestDataPtr.IsValid())
			{
				TAudioModulationPtr ModPtr = ModInterfacePtr.Pin();
				if (ModPtr.IsValid())
				{
					TSet<FModulatorHandle> NewHandles;
					for (TUniquePtr<Audio::IModulatorSettings>& ModSetting : ModSettings)
					{
						Audio::FModulationParameter HandleParam = DestDataPtr->GetParameter();
						NewHandles.Add(FModulatorHandle { *ModPtr.Get(), *ModSetting.Get(), MoveTemp(HandleParam) });
					}
					DestDataPtr->SetHandles(MoveTemp(NewHandles));
					return;
				}
				DestDataPtr->ResetHandles();
			}
		});
	}
} // namespace Audio

