// Copyright Epic Games, Inc. All Rights Reserved.


#include "Sound/SoundNode.h"
#include "EdGraph/EdGraphNode.h"
#include "EngineUtils.h"
#include "Sound/SoundCue.h"
#include "Misc/App.h"
#include "Sound/SoundNodeWavePlayer.h"
#include "Sound/SoundNodeQualityLevel.h"
#include "AudioCompressionSettingsUtils.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SoundNode)

static int32 BypassRetainInSoundNodesCVar = 0;
FAutoConsoleVariableRef CVarBypassRetainInSoundNodes(
	TEXT("au.streamcache.priming.BypassRetainFromSoundCues"),
	BypassRetainInSoundNodesCVar,
	TEXT("When set to 1, we ignore the loading behavior of sound classes set on a Sound Cue directly.\n"),
	ECVF_Default);

static int32 ManuallyPrimeChildNodesCVar = 1;
FAutoConsoleVariableRef CVarManuallyPrimeChildNodes(			
	TEXT("au.streamcache.priming.ManuallyPrimeChildNodes"),
	ManuallyPrimeChildNodesCVar,			
	TEXT("When set to 1, we ignore the loading behavior of sound classes set on a Sound Cue directly.\n"),
	ECVF_Default);

/*-----------------------------------------------------------------------------
	USoundNode implementation.
-----------------------------------------------------------------------------*/
USoundNode::USoundNode(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	RandomStream.Initialize(FApp::bUseFixedSeed ? GetFName() : NAME_None);
	bIsRetainingAudio = false;
}


void USoundNode::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

	if (Ar.UEVer() >= VER_UE4_COOKED_ASSETS_IN_EDITOR_SUPPORT)
	{
		FStripDataFlags StripFlags(Ar);
#if WITH_EDITORONLY_DATA
		if (!StripFlags.IsEditorDataStripped())
		{
			Ar << GraphNode;
		}
#endif
	}
#if WITH_EDITOR
	else
	{
		Ar << GraphNode;
	}
#endif
}

bool USoundNode::CanBeClusterRoot() const
{
	return false;
}

bool USoundNode::CanBeInCluster() const
{
	return true;
}

#if WITH_EDITOR
void USoundNode::AddReferencedObjects(UObject* InThis, FReferenceCollector& Collector)
{
	USoundNode* This = CastChecked<USoundNode>(InThis);
	Collector.AddReferencedObject(This->GraphNode, This);

	Super::AddReferencedObjects(InThis, Collector);
}
#endif //WITH_EDITOR

#if WITH_EDITORONLY_DATA
UEdGraphNode* USoundNode::GetGraphNode() const
{
	return GraphNode;
}
#endif

UPTRINT USoundNode::GetNodeWaveInstanceHash(const UPTRINT ParentWaveInstanceHash, const USoundNode* ChildNode, const uint32 ChildIndex)
{
	checkf(ChildIndex < MAX_ALLOWED_CHILD_NODES, TEXT("Too many children (%d) in SoundCue '%s'"), ChildIndex, *CastChecked<USoundCue>(ChildNode->GetOuter())->GetFullName());

	return GetNodeWaveInstanceHash(ParentWaveInstanceHash, reinterpret_cast<const UPTRINT>(ChildNode), ChildIndex);
}

UPTRINT USoundNode::GetNodeWaveInstanceHash(const UPTRINT ParentWaveInstanceHash, const UPTRINT ChildNodeHash, const uint32 ChildIndex)
{
#define USE_NEW_SOUNDCUE_NODE_HASH 1
#if USE_NEW_SOUNDCUE_NODE_HASH
	const uint32 ChildHash = PointerHash(reinterpret_cast<const void*>(ChildNodeHash), GetTypeHash(ChildIndex));
	const uint32 Hash = PointerHash(reinterpret_cast<const void*>(ParentWaveInstanceHash), ChildHash);

	return static_cast<UPTRINT>(Hash);
#else
	return ((ParentWaveInstanceHash << ChildIndex) ^ ChildNodeHash);
#endif // USE_NEW_SOUNDCUE_NODE_HASH
}

void USoundNode::PrimeChildWavePlayers(bool bRecurse)
{
	// Note: it is not safe to call IAudioStreamingManager::RequestChunk from the game thread
	// if there is no async loading thread.  This can deadlock:

	// The call will try to obtain a lock held by another thread, that thread may be waiting for
	// any pending async loads to complete, which will never happen if WE are the (non)"async loading thread"
	// for now, in the synchronous loading case, we fallback to the old behavior.
	if(!BypassRetainInSoundNodesCVar && ManuallyPrimeChildNodesCVar && IsAsyncLoadingMultithreaded())
	{	
		// Search child nodes for wave players, then prime each soundwave
		IAudioStreamingManager &Mgr = IStreamingManager::Get().GetAudioStreamingManager();
		for (USoundNode* ChildNode : ChildNodes)
		{
			if (ChildNode)
			{
				ChildNode->ConditionalPostLoad();
				if (bRecurse)
				{
					ChildNode->PrimeChildWavePlayers(true);
				}

				if (USoundNodeWavePlayer* WavePlayer = Cast<USoundNodeWavePlayer>(ChildNode))
				{
					if (USoundWave* SoundWave = WavePlayer->GetSoundWave())
					{
						TSharedRef<const FSoundWaveProxy> SoundWaveProxy = SoundWave->GetSoundWaveProxy();
						if (SoundWave->IsStreaming() && SoundWave->GetNumChunks() > 1)
						{
							Mgr.RequestChunk(SoundWaveProxy->GetSoundWaveDataRef(), 1, [](EAudioChunkLoadResult) {});
						}
					}
				}
			}
		}
	}
	else
	{
		OverrideLoadingBehaviorOnChildWaves(bRecurse, ESoundWaveLoadingBehavior::PrimeOnLoad);
	}
}

void USoundNode::RetainChildWavePlayers(bool bRecurse)
{
	OverrideLoadingBehaviorOnChildWaves(bRecurse, ESoundWaveLoadingBehavior::RetainOnLoad);
}

void USoundNode::OverrideLoadingBehaviorOnChildWaves(const bool bRecurse, const ESoundWaveLoadingBehavior InLoadingBehavior)
{
	if (!BypassRetainInSoundNodesCVar)
	{
		// Search child nodes for wave players, then override their waves' loading behavior.
		for (USoundNode* ChildNode : ChildNodes)
		{
			if (ChildNode)
			{
				ChildNode->ConditionalPostLoad();
				if (bRecurse)
				{
					ChildNode->OverrideLoadingBehaviorOnChildWaves(true, InLoadingBehavior);
				}

				if (USoundNodeWavePlayer* WavePlayer = Cast<USoundNodeWavePlayer>(ChildNode))
				{
					USoundWave* SoundWave = WavePlayer->GetSoundWave();
					if (SoundWave)
					{
						SoundWave->OverrideLoadingBehavior(InLoadingBehavior);
					}
				}
			}
		}
	}

	if (!GIsEditor && InLoadingBehavior == ESoundWaveLoadingBehavior::RetainOnLoad)
	{
		bIsRetainingAudio = true;
	}
}

void USoundNode::ReleaseRetainerOnChildWavePlayers(bool bRecurse)
{
	if (bIsRetainingAudio)
	{
		// Search child nodes for wave players, then release their retainers.
		for (USoundNode* ChildNode : ChildNodes)
		{
			if (ChildNode)
			{
				ChildNode->ConditionalPostLoad();
				if (bRecurse)
				{
					ChildNode->ReleaseRetainerOnChildWavePlayers(true);
				}

				USoundNodeWavePlayer* WavePlayer = Cast<USoundNodeWavePlayer>(ChildNode);
				if (WavePlayer != nullptr)
				{
					USoundWave* SoundWave = WavePlayer->GetSoundWave();
					if (SoundWave)
					{
						SoundWave->ReleaseCompressedAudio();
					}
				}
			}
		}
	}

	bIsRetainingAudio = false;
}

void USoundNode::RemoveSoundWaveOnChildWavePlayers()
{
	// Search child nodes for wave players, then null out their sound wave.
	for (USoundNode* ChildNode : ChildNodes)
	{
		if (ChildNode)
		{
			ChildNode->ConditionalPostLoad();
			ChildNode->RemoveSoundWaveOnChildWavePlayers();

			USoundNodeWavePlayer* WavePlayer = Cast<USoundNodeWavePlayer>(ChildNode);
			if (WavePlayer != nullptr)
			{
				USoundWave* SoundWave = WavePlayer->GetSoundWave();
				if (SoundWave && !WavePlayer->IsCurrentlyAsyncLoadingAsset())
				{
					WavePlayer->ClearAssetReferences();
				}
			}
		}
	}
}

void USoundNode::LoadChildWavePlayerAssets(bool bAddToRoot, bool bRecurse)
{
	// Search child nodes for wave players, then load their sound wave asset.
	for (USoundNode* ChildNode : ChildNodes)
	{
		if (ChildNode)
		{
			if (bRecurse)
			{
				ChildNode->LoadChildWavePlayerAssets(bAddToRoot, bRecurse);
			}

			if (USoundNodeWavePlayer* WavePlayer = Cast<USoundNodeWavePlayer>(ChildNode))
			{
				WavePlayer->LoadAsset(bAddToRoot);
			}
			else if (USoundNodeQualityLevel* QualityNode = Cast<USoundNodeQualityLevel>(ChildNode))
			{
				// Take into account quality nodes by only loading wave players for the relevant quality level
				QualityNode->LoadChildWavePlayers(bAddToRoot, bRecurse);
			}
		}
	}
}

void USoundNode::BeginDestroy()
{
	Super::BeginDestroy();
}

void USoundNode::ParseNodes( FAudioDevice* AudioDevice, const UPTRINT NodeWaveInstanceHash, FActiveSound& ActiveSound, const FSoundParseParameters& ParseParams, TArray<FWaveInstance*>& WaveInstances )
{
	for( int32 i = 0; i < ChildNodes.Num() && i < GetMaxChildNodes(); ++i )
	{
		if( ChildNodes[ i ] )
		{
			ChildNodes[ i ]->ParseNodes( AudioDevice, GetNodeWaveInstanceHash(NodeWaveInstanceHash, ChildNodes[i], i), ActiveSound, ParseParams, WaveInstances );
		}
	}
}

void USoundNode::GetAllNodes( TArray<USoundNode*>& SoundNodes )
{
	SoundNodes.Add( this );
	for( int32 i = 0; i < ChildNodes.Num(); ++i )
	{
		if( ChildNodes[ i ] )
		{
			ChildNodes[ i ]->GetAllNodes( SoundNodes );
		}
	}
}

void USoundNode::CreateStartingConnectors()
{
	int32 ConnectorsToMake = FMath::Max(1, GetMinChildNodes());
	while (ConnectorsToMake > 0)
	{
		InsertChildNode( ChildNodes.Num() );
		--ConnectorsToMake;
	}
}

void USoundNode::InsertChildNode( int32 Index )
{
	check( Index >= 0 && Index <= ChildNodes.Num() );
	int32 MaxChildNodes = GetMaxChildNodes();
	if (MaxChildNodes > ChildNodes.Num())
	{
		ChildNodes.InsertZeroed( Index );
#if WITH_EDITOR
		USoundCue::GetSoundCueAudioEditor()->CreateInputPin(GetGraphNode());
#endif //WITH_EDITORONLY_DATA
	}
}

void USoundNode::RemoveChildNode( int32 Index )
{
	check( Index >= 0 && Index < ChildNodes.Num() );
	int32 MinChildNodes = GetMinChildNodes();
	if (ChildNodes.Num() > MinChildNodes )
	{
		ChildNodes.RemoveAt( Index );
	}
}

#if WITH_EDITOR
void USoundNode::SetChildNodes(TArray<USoundNode*>& InChildNodes)
{
	int32 MaxChildNodes = GetMaxChildNodes();
	int32 MinChildNodes = GetMinChildNodes();
	if (MaxChildNodes >= InChildNodes.Num() && InChildNodes.Num() >= MinChildNodes)
	{
		ChildNodes = InChildNodes;
	}
}
#endif //WITH_EDITOR

float USoundNode::GetDuration()
{
	// Iterate over children and return maximum length of any of them
	float MaxDuration = 0.0f;
	for (USoundNode* ChildNode : ChildNodes)
	{
		if (ChildNode)
		{
			ChildNode->ConditionalPostLoad();
			MaxDuration = FMath::Max(ChildNode->GetDuration(), MaxDuration);
		}
	}
	return MaxDuration;
}

float USoundNode::GetMaxDistance() const
{
	float MaxDistance = 0.0f;
	for (USoundNode* ChildNode : ChildNodes)
	{
		if (ChildNode)
		{
			ChildNode->ConditionalPostLoad();
			MaxDistance = FMath::Max(ChildNode->GetMaxDistance(), MaxDistance);
		}
	}
	return MaxDistance;
}

bool USoundNode::HasDelayNode() const
{
	for (USoundNode* ChildNode : ChildNodes)
	{
		if (ChildNode)
		{
			ChildNode->ConditionalPostLoad();
			if (ChildNode->HasDelayNode())
			{
				return true;
			}
		}
	}
	return false;
}

bool USoundNode::HasConcatenatorNode() const
{
	for (USoundNode* ChildNode : ChildNodes)
	{
		if (ChildNode)
		{
			ChildNode->ConditionalPostLoad();
			if (ChildNode->HasConcatenatorNode())
			{
				return true;
			}
		}
	}
	return false;
}

bool USoundNode::HasAttenuationNode() const
{
	for (USoundNode* ChildNode : ChildNodes)
	{
		if (ChildNode)
		{
			ChildNode->ConditionalPostLoad();
			if (ChildNode->HasAttenuationNode())
			{
				return true;
			}
		}
	}
	return false;
}

bool USoundNode::IsPlayWhenSilent() const
{
	for (USoundNode* ChildNode : ChildNodes)
	{
		if (ChildNode)
		{
			ChildNode->ConditionalPostLoad();
			if (ChildNode->IsPlayWhenSilent())
			{
				return true;
			}
		}
	}
	return false;
}

int32 USoundNode::GetNumSounds(const UPTRINT NodeWaveInstanceHash, FActiveSound& ActiveSound) const
{
	// Default implementation loops through all child nodes and sums the number of sounds.
	// For most nodes this will result in 1, for node mixers, this will result in multiple sounds.
	int32 NumSounds = 0;
	for (int32 i = 0; i < ChildNodes.Num(); ++i)
	{
		if (ChildNodes[i])
		{
			const UPTRINT ChildNodeWaveInstanceHash = GetNodeWaveInstanceHash(NodeWaveInstanceHash, ChildNodes[i], i);
			NumSounds += ChildNodes[i]->GetNumSounds(ChildNodeWaveInstanceHash, ActiveSound);
		}
	}

	return NumSounds;
}

#if WITH_EDITOR
void USoundNode::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	MarkPackageDirty();
}

void USoundNode::PostLoad()
{
	Super::PostLoad();
	// Make sure sound nodes are transactional (so they work with undo system)
	SetFlags(RF_Transactional);
}
#endif // WITH_EDITOR

#if WITH_EDITOR
void USoundNode::PlaceNode( int32 NodeColumn, int32 NodeRow, int32 RowCount )
{
	GraphNode->NodePosX = (-150 * NodeColumn) - 100;
	GraphNode->NodePosY = (100 * NodeRow) - (50 * RowCount);
}

#endif //WITH_EDITOR

