// Copyright Epic Games, Inc. All Rights Reserved.

#include "Sound/SoundNodeAssetReferencer.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SoundNodeAssetReferencer)

#define ASYNC_LOAD_RANDOMIZED_SOUNDS 1

// If stream caching is enabled and this is set to 0,
// we may result in dropped sounds due to USoundNodeWavePlayer::LoadAsset nulling out the SoundWave ptr.
#define MAKE_SOUNDWAVES_HARD_REFERENCES 1

bool USoundNodeAssetReferencer::ShouldHardReferenceAsset(const ITargetPlatform* TargetPlatform) const
{	
#if MAKE_SOUNDWAVES_HARD_REFERENCES
	return true;
#else
	if (TargetPlatform)
	{
		if (const FPlatformAudioCookOverrides* CookOverrides = FPlatformCompressionUtilities::GetCookOverrides(*TargetPlatform->IniPlatformName()))
		{
			// If the Quality nodes are cooked, everything is hard refs.
			if (CookOverrides->SoundCueCookQualityIndex != INDEX_NONE)
			{
				UE_LOGF(LogAudio, Verbose, "HARD reffing '%ls:%ls', as we are cooking using quality '%ls'",
					*GetNameSafe(GetOuter()),
					*GetName(),
					*GetDefault<UAudioSettings>()->FindQualityNameByIndex(CookOverrides->SoundCueCookQualityIndex)
				)
				return true;
			}
		}
	}

	bool bShouldHardReference = true;	
	if (USoundCue* Cue = Cast<USoundCue>(GetOuter()))
	{
		TArray<USoundNodeQualityLevel*> QualityNodes;
		TArray<USoundNodeAssetReferencer*> WavePlayers;
		Cue->RecursiveFindNode(Cue->FirstNode, QualityNodes);

		for (USoundNodeQualityLevel* QualityNode : QualityNodes)
		{
			WavePlayers.Reset();
			Cue->RecursiveFindNode(QualityNode, WavePlayers);
			if (WavePlayers.Contains(this))
			{
				bShouldHardReference = false;
				break;
			}
		}

#if ASYNC_LOAD_RANDOMIZED_SOUNDS
		if (bShouldHardReference)
		{
			//Check for randomized sounds as well:
			TArray<USoundNodeRandom*> RandomNodes;
			Cue->RecursiveFindNode(Cue->FirstNode, RandomNodes);

			for (USoundNodeRandom* RandomNode : RandomNodes)
			{
				WavePlayers.Reset();
				Cue->RecursiveFindNode(RandomNode, WavePlayers);
				if (WavePlayers.Contains(this))
				{
					bShouldHardReference = false;
					break;
				}
			}
		}
#endif // ASYNC_LOAD_RANDOMIZED_SOUNDS
	}

	UE_LOGF(LogAudio, Verbose, "%ls reffing '%ls:%ls'.",
		bShouldHardReference ? TEXT("HARD") : TEXT("SOFT"),
		*GetNameSafe(GetOuter()),
		*GetName()
	);

	return bShouldHardReference;
#endif // MAKE_SOUNDWAVES_HARD_REFERENCES
}

#if WITH_EDITOR
void USoundNodeAssetReferencer::PostEditImport()
{
	Super::PostEditImport();

	LoadAsset();
}
#endif

