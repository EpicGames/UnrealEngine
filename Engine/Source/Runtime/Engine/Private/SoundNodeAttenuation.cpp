// Copyright Epic Games, Inc. All Rights Reserved.


#include "Sound/SoundNodeAttenuation.h"
#include "AudioDevice.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SoundNodeAttenuation)

/*-----------------------------------------------------------------------------
	USoundNodeAttenuation implementation.
-----------------------------------------------------------------------------*/

USoundNodeAttenuation::USoundNodeAttenuation(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

float USoundNodeAttenuation::GetMaxDistance() const 
{ 
	float MaxDistance = FAudioDevice::GetMaxWorldDistance();
	const FSoundAttenuationSettings* Settings = GetAttenuationSettingsToApply();
	if (Settings)
	{
		MaxDistance = Settings->GetMaxDimension();
	}

	for (USoundNode* ChildNode : ChildNodes)
	{
		if (ChildNode)
		{
			ChildNode->ConditionalPostLoad();
			MaxDistance = FMath::Max(ChildNode->GetMaxDistance(), MaxDistance);
		}
	}
	return MaxDistance;
}

const FSoundAttenuationSettings* USoundNodeAttenuation::GetAttenuationSettingsToApply() const
{
	const FSoundAttenuationSettings* Settings = nullptr;

	if (bOverrideAttenuation)
	{
		Settings = &AttenuationOverrides;
	}
	else if (AttenuationSettings)
	{
		Settings = &AttenuationSettings->Attenuation;
	}

	return Settings;
}

void USoundNodeAttenuation::ParseNodes(FAudioDevice* AudioDevice, const UPTRINT NodeWaveInstanceHash, FActiveSound& ActiveSound, const FSoundParseParameters& ParseParams, TArray<FWaveInstance*>& WaveInstances)
{
	FSoundParseParameters UpdatedParseParams = ParseParams;

	const FSoundAttenuationSettings* Settings = (ActiveSound.bAllowSpatialization ? GetAttenuationSettingsToApply() : nullptr);
	if (Settings)
	{
		// Update this node's attenuation settings overrides
		check(AudioDevice);
		const int32 ClosestListenerIndex = AudioDevice->FindClosestListenerIndex(UpdatedParseParams.Transform);
		ActiveSound.ParseAttenuation(UpdatedParseParams, ClosestListenerIndex, *Settings);
	}
	else
	{
		UpdatedParseParams.bUseSpatialization = false;
	}

	Super::ParseNodes(AudioDevice, NodeWaveInstanceHash, ActiveSound, UpdatedParseParams, WaveInstances);
}
