// Copyright Epic Games, Inc. All Rights Reserved.


#include "Sound/SoundNodeBranch.h"
#include "ActiveSound.h"
#include "IAudioParameterTransmitter.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SoundNodeBranch)

#define LOCTEXT_NAMESPACE "SoundNodeBranch"

/*-----------------------------------------------------------------------------
    USoundNodeLocality implementation.
-----------------------------------------------------------------------------*/
USoundNodeBranch::USoundNodeBranch(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

void USoundNodeBranch::ParseNodes(FAudioDevice* AudioDevice, const UPTRINT NodeWaveInstanceHash, FActiveSound& ActiveSound, const FSoundParseParameters& ParseParams, TArray<FWaveInstance*>& WaveInstances)
{
	BranchPurpose BranchToUse = BranchPurpose::ParameterUnset;

	FAudioParameter ParamValue;
	if (ActiveSound.GetTransmitter()->GetParameter(BoolParameterName, ParamValue))
	{
		BranchToUse = (ParamValue.BoolParam ? BranchPurpose::ParameterTrue : BranchPurpose::ParameterFalse);
	}

	const int32 ChildNodeIndex = (int32)BranchToUse;
	if (ChildNodeIndex < ChildNodes.Num() && ChildNodes[ChildNodeIndex])
	{
		ChildNodes[ChildNodeIndex]->ParseNodes(AudioDevice, GetNodeWaveInstanceHash(NodeWaveInstanceHash, ChildNodes[ChildNodeIndex], ChildNodeIndex), ActiveSound, ParseParams, WaveInstances);
	}
}

#if WITH_EDITOR
FText USoundNodeBranch::GetInputPinName(int32 PinIndex) const
{
	const BranchPurpose Branch = (BranchPurpose)PinIndex;
	switch (Branch)
	{
	case BranchPurpose::ParameterTrue:
		return LOCTEXT("True", "True");

	case BranchPurpose::ParameterFalse:
		return LOCTEXT("False", "False");

	case BranchPurpose::ParameterUnset:
		return LOCTEXT("ParamUnset", "Parameter Unset");
	}

	return Super::GetInputPinName(PinIndex);
}

FText USoundNodeBranch::GetTitle() const
{
	FFormatNamedArguments Arguments;
	Arguments.Add(TEXT("Description"), Super::GetTitle());
	Arguments.Add(TEXT("ParameterName"), FText::FromName(BoolParameterName));

	return FText::Format(LOCTEXT("Title", "{Description} ({ParameterName})"), Arguments);
}
#endif //WITH_EDITOR

#undef LOCTEXT_NAMESPACE

