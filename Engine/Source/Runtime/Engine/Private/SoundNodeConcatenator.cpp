// Copyright Epic Games, Inc. All Rights Reserved.


#include "Sound/SoundNodeConcatenator.h"
#include "ActiveSound.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SoundNodeConcatenator)

// Payload used for instances of a concatenator node
struct FSoundNodeConcatenatorPayload
{
	/** Which child node we're currently playing back in the concatenation sequence */
	int32 NodeIndex;

	/** Which child node sound index we're currently playing in the current child node */
	int32 CurrentChildNodeSoundIndex;

	/** The number of sounds playing in current child node */
	int32 CurrentChildNodeNumSound;
};


/*-----------------------------------------------------------------------------
         USoundNodeConcatenator implementation.
-----------------------------------------------------------------------------*/
USoundNodeConcatenator::USoundNodeConcatenator(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

bool USoundNodeConcatenator::NotifyWaveInstanceFinished( FWaveInstance* WaveInstance )
{
	FActiveSound& ActiveSound = *WaveInstance->ActiveSound;
	const UPTRINT NodeWaveInstanceHash = WaveInstance->NotifyBufferFinishedHooks.GetHashForNode(this);
	RETRIEVE_SOUNDNODE_PAYLOAD(sizeof(FSoundNodeConcatenatorPayload));
	DECLARE_SOUNDNODE_ELEMENT(FSoundNodeConcatenatorPayload, ConcatenatorPayload);
	check( *RequiresInitialization == 0 );

	// Only bump up the node index if all the current child node's sounds have finished playing. 
	// Otherwise, this will end up cutting short sounds played through a mixer.

	ConcatenatorPayload.CurrentChildNodeSoundIndex++;
	if (ConcatenatorPayload.NodeIndex < ChildNodes.Num())
	{
		if (ConcatenatorPayload.CurrentChildNodeSoundIndex == ConcatenatorPayload.CurrentChildNodeNumSound)
		{
			// Find the next non-null index
			for (++ConcatenatorPayload.NodeIndex; ConcatenatorPayload.NodeIndex < ChildNodes.Num(); ++ConcatenatorPayload.NodeIndex)
			{
				if (ChildNodes[ConcatenatorPayload.NodeIndex])
				{
					break;
				}
			}

			ConcatenatorPayload.CurrentChildNodeSoundIndex = 0;

			// Allow wave instance to be played again the next iteration.
			WaveInstance->bIsStarted = false;
			WaveInstance->bIsFinished = false;
		}
	}

	return (ConcatenatorPayload.NodeIndex < ChildNodes.Num());
}


float USoundNodeConcatenator::GetDuration()
{
	// Sum up length of child nodes.
	float Duration = 0.0f;
	for (int32 ChildNodeIndex = 0; ChildNodeIndex < ChildNodes.Num(); ChildNodeIndex++)
	{
		USoundNode* ChildNode = ChildNodes[ChildNodeIndex];
		if (ChildNode)
		{
			Duration += ChildNode->GetDuration();
		}
	}
	return Duration;
}

int32 USoundNodeConcatenator::GetNumSounds(const UPTRINT NodeWaveInstanceHash, FActiveSound& ActiveSound) const
{
	// Counter-intuitively, a node concatenator will always play 1 sound from the perspective of other node-concatenators
	return 1;
}

void USoundNodeConcatenator::CreateStartingConnectors()
{
	// Concatenators default to two two connectors.
	InsertChildNode( ChildNodes.Num() );
	InsertChildNode( ChildNodes.Num() );
}


void USoundNodeConcatenator::InsertChildNode( int32 Index )
{
	Super::InsertChildNode( Index );
	InputVolume.InsertUninitialized( Index );
	InputVolume[ Index ] = 1.0f;
}


void USoundNodeConcatenator::RemoveChildNode( int32 Index )
{
	Super::RemoveChildNode( Index );
	InputVolume.RemoveAt( Index );
}

void USoundNodeConcatenator::ParseNodes( FAudioDevice* AudioDevice, const UPTRINT NodeWaveInstanceHash, FActiveSound& ActiveSound, const FSoundParseParameters& ParseParams, TArray<FWaveInstance*>& WaveInstances )
{
	// Local stack copy of the payload. We need to do this because ParseNodes may resize the payload map
	FSoundNodeConcatenatorPayload LocalPayload;

	{
		RETRIEVE_SOUNDNODE_PAYLOAD(sizeof(FSoundNodeConcatenatorPayload));
		DECLARE_SOUNDNODE_ELEMENT(FSoundNodeConcatenatorPayload, ConcatenatorPayload);

		// Start from the beginning.
		if (*RequiresInitialization)
		{
			ConcatenatorPayload.NodeIndex = 0;
			ConcatenatorPayload.CurrentChildNodeSoundIndex = 0;
			ConcatenatorPayload.CurrentChildNodeNumSound = 0;
			*RequiresInitialization = false;
		}

		ConcatenatorPayload.CurrentChildNodeNumSound = 0;

		LocalPayload = ConcatenatorPayload;
	}

	// Play the current node.
	while (LocalPayload.NodeIndex < ChildNodes.Num())
	{
		USoundNode* ChildNode = ChildNodes[LocalPayload.NodeIndex];
		if (ChildNode)
		{
			// Play currently active node.
			FSoundParseParameters UpdatedParams = ParseParams;
			UpdatedParams.NotifyBufferFinishedHooks.AddNotify(this, NodeWaveInstanceHash);
			UpdatedParams.VolumeMultiplier *= InputVolume[LocalPayload.NodeIndex];

			const int32 NumWaveInstancesBeforeParse = WaveInstances.Num();

			const UPTRINT ChildNodeWaveInstanceHash = GetNodeWaveInstanceHash(NodeWaveInstanceHash, ChildNode, LocalPayload.NodeIndex);
			ChildNode->ParseNodes(AudioDevice, ChildNodeWaveInstanceHash, ActiveSound, UpdatedParams, WaveInstances);

			// Check to see if we actually added any wave instances when parsing the concat node.
			if (WaveInstances.Num() > NumWaveInstancesBeforeParse)
			{
				const int32 NumChildSounds = ChildNode->GetNumSounds(ChildNodeWaveInstanceHash, ActiveSound);
				
				// Update the payload with the number of sounds played and update our local copy
				RETRIEVE_SOUNDNODE_PAYLOAD(sizeof(FSoundNodeConcatenatorPayload));
				DECLARE_SOUNDNODE_ELEMENT(FSoundNodeConcatenatorPayload, ConcatenatorPayload);

				ConcatenatorPayload = LocalPayload;
				ConcatenatorPayload.CurrentChildNodeNumSound = NumChildSounds;

				break;
			}
		}

		// If no child node, or if no wave instances played for this node index
		LocalPayload.CurrentChildNodeSoundIndex = 0;
		++LocalPayload.NodeIndex;
	}
}

#if WITH_EDITOR
void USoundNodeConcatenator::SetChildNodes(TArray<USoundNode*>& InChildNodes)
{
	Super::SetChildNodes(InChildNodes);

	if (InputVolume.Num() < ChildNodes.Num())
	{
		while (InputVolume.Num() < ChildNodes.Num())
		{
			int32 NewIndex = InputVolume.Num();
			InputVolume.InsertUninitialized(NewIndex);
			InputVolume[ NewIndex ] = 1.0f;
		}
	}
	else if (InputVolume.Num() > ChildNodes.Num())
	{
		const int32 NumToRemove = InputVolume.Num() - ChildNodes.Num();
		InputVolume.RemoveAt(InputVolume.Num() - NumToRemove, NumToRemove);
	}
}
#endif //WITH_EDITOR

