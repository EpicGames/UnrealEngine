// Copyright Epic Games, Inc. All Rights Reserved.


#include "Sound/SoundNodeDelay.h"
#include "ActiveSound.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SoundNodeDelay)

static int32 PrimeDelayedSoundNodesCVar = 0;
FAutoConsoleVariableRef CVarPrimeDelayedSoundNodes(
	TEXT("au.streamcache.priming.PrimeDelayNodes"),
	PrimeDelayedSoundNodesCVar,
	TEXT("When set to 1, sounds will be loaded into the cache automatically when a delay node is hit.\n"),
	ECVF_Default);

struct FSoundNodeDelayPayload
{
	float EndOfDelay;
	float StartTimeModifier;
};

/*-----------------------------------------------------------------------------
         USoundNodeDelay implementation.
-----------------------------------------------------------------------------*/
USoundNodeDelay::USoundNodeDelay(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	DelayMin = 0.0f;
	DelayMax = 0.0f;

}

void USoundNodeDelay::ParseNodes( FAudioDevice* AudioDevice, const UPTRINT NodeWaveInstanceHash, FActiveSound& ActiveSound, const FSoundParseParameters& ParseParams, TArray<FWaveInstance*>& WaveInstances )
{
	RETRIEVE_SOUNDNODE_PAYLOAD( sizeof(FSoundNodeDelayPayload) );
	DECLARE_SOUNDNODE_ELEMENT(FSoundNodeDelayPayload, SoundNodeDelayPayload );

	// Check to see if this is the first time through.
	if( *RequiresInitialization )
	{
		*RequiresInitialization = false;

		const float ActualDelay = FMath::Max(0.f, DelayMax + ( ( DelayMin - DelayMax ) * RandomStream.FRand() ));

		if (PrimeDelayedSoundNodesCVar != 0)
		{
			PrimeChildWavePlayers(true);
		}

		if (ActualDelay > 0.0f && ParseParams.StartTime >= ActualDelay)
		{
			SoundNodeDelayPayload.StartTimeModifier = ActualDelay;
			SoundNodeDelayPayload.EndOfDelay = -1.f;

			FSoundParseParameters UpdatedParams = ParseParams;
			UpdatedParams.StartTime -= SoundNodeDelayPayload.StartTimeModifier;
					
			Super::ParseNodes( AudioDevice, NodeWaveInstanceHash, ActiveSound, UpdatedParams, WaveInstances );
			return;
		}
		else
		{
			SoundNodeDelayPayload.StartTimeModifier = 0.0f;
			SoundNodeDelayPayload.EndOfDelay = ActiveSound.PlaybackTime + ActualDelay - ParseParams.StartTime;
		}
	}

	// If we have not waited long enough then just keep waiting.
	if (SoundNodeDelayPayload.EndOfDelay > ActiveSound.PlaybackTime )
	{
		// We're not finished even though we might not have any wave instances in flight.
		ActiveSound.bFinished = false;
	}
	// Go ahead and play the sound.
	else
	{
		FSoundParseParameters UpdatedParams = ParseParams;
		UpdatedParams.StartTime -= SoundNodeDelayPayload.StartTimeModifier;

		Super::ParseNodes( AudioDevice, NodeWaveInstanceHash, ActiveSound, UpdatedParams, WaveInstances );
	}
}

float USoundNodeDelay::GetDuration()
{
	// Get length of child node.
	float ChildDuration = 0.0f;
	if (ChildNodes[0])
	{
		ChildDuration = ChildNodes[0]->GetDuration();
	}

	// And return the two together.
	return ChildDuration + DelayMax;
}

