// Copyright Epic Games, Inc. All Rights Reserved.

#include "Sound/SoundNodeDialoguePlayer.h"
#include "ActiveSound.h"
#include "Sound/DialogueWave.h"
#include "Sound/DialogueSoundWaveProxy.h"

FOnDialogueSoundWaveProxyParsed USoundNodeDialoguePlayer::OnDialogueSoundWaveProxyParsed;

#include UE_INLINE_GENERATED_CPP_BY_NAME(SoundNodeDialoguePlayer)

#define LOCTEXT_NAMESPACE "SoundNodeDialoguePlayer"

USoundNodeDialoguePlayer::USoundNodeDialoguePlayer(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

void USoundNodeDialoguePlayer::ParseNodes( FAudioDevice* AudioDevice, const UPTRINT NodeWaveInstanceHash, FActiveSound& ActiveSound, const FSoundParseParameters& ParseParams, TArray<FWaveInstance*>& WaveInstances )
{
	USoundBase* SoundBase = GetDialogueWave() ? GetDialogueWave()->GetWaveFromContext(DialogueWaveParameter.Context) : NULL;
	if (SoundBase)
	{
		RETRIEVE_SOUNDNODE_PAYLOAD(sizeof(uint8));
		if (*RequiresInitialization)
		{
			*RequiresInitialization = 0;
			if (const UDialogueSoundWaveProxy* Proxy = Cast<UDialogueSoundWaveProxy>(SoundBase))
			{
				OnDialogueSoundWaveProxyParsed.Broadcast(ActiveSound, Proxy);
			}
		}

		if (bLooping)
		{
			FSoundParseParameters UpdatedParams = ParseParams;
			UpdatedParams.bLooping = true;
			SoundBase->Parse(AudioDevice, NodeWaveInstanceHash, ActiveSound, UpdatedParams, WaveInstances);
		}
		else
		{
			SoundBase->Parse(AudioDevice, NodeWaveInstanceHash, ActiveSound, ParseParams, WaveInstances);
		}
	}
}

float USoundNodeDialoguePlayer::GetDuration()
{
	USoundBase* SoundBase = GetDialogueWave() ? GetDialogueWave()->GetWaveFromContext(DialogueWaveParameter.Context) : NULL;
	float Duration = 0.f;
	if (SoundBase)
	{
		if (bLooping)
		{
			Duration = INDEFINITELY_LOOPING_DURATION;
		}
		else
		{
			Duration = SoundBase->GetDuration();
		}
	}
	return Duration;
}

// A Wave Player is the end of the chain and has no children
int32 USoundNodeDialoguePlayer::GetMaxChildNodes() const
{
	return 0;
}

#if WITH_EDITOR
FText USoundNodeDialoguePlayer::GetTitle() const
{
	FText DialogueWaveName;
	if (GetDialogueWave())
	{
		DialogueWaveName = FText::FromString(GetDialogueWave()->GetFName().ToString());
	}
	else
	{
		DialogueWaveName = LOCTEXT("NoDialogueWave", "NONE");
	}

	FText Title;

	FFormatNamedArguments Arguments;
	Arguments.Add(TEXT("Description"), Super::GetTitle());
	Arguments.Add(TEXT("DialogueWaveName"), DialogueWaveName);
	if (bLooping)
	{
		Title = FText::Format(LOCTEXT("LoopingDialogueWaveDescription", "Looping {Description} : {DialogueWaveName}"), Arguments);
	}
	else
	{
		Title = FText::Format(LOCTEXT("NonLoopingDialogueWaveDescription", "{Description} : {DialogueWaveName}"), Arguments);
	}

	return Title;
}
#endif

void USoundNodeDialoguePlayer::SetDialogueWave(UDialogueWave* Value)
{
	DialogueWaveParameter.DialogueWave = Value;
}

const UDialogueSoundWaveProxy* USoundNodeDialoguePlayer::GetContextualDialogueSoundWaveProxy() const
{
	const UDialogueWave* DialogueWave = GetDialogueWave();

	if (IsValid(DialogueWave))
	{
		const USoundBase* SoundBase = DialogueWave ? DialogueWave->GetWaveFromContext(DialogueWaveParameter.Context) : NULL;
		if (IsValid(SoundBase))
		{
			const UDialogueSoundWaveProxy* Proxy = Cast<UDialogueSoundWaveProxy>(SoundBase);
			check(IsValid(Proxy));	// GetWaveFromContext() specifically uses DialogueSoundWaveProxy, but casts to SoundBase before returning.
			return Proxy;
		}
	}
	return nullptr;
}

UDialogueWave* USoundNodeDialoguePlayer::GetDialogueWave() const
{
	return DialogueWaveParameter.DialogueWave;
}

#undef LOCTEXT_NAMESPACE

