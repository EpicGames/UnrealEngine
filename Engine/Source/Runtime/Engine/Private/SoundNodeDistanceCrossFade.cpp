// Copyright Epic Games, Inc. All Rights Reserved.


#include "Sound/SoundNodeDistanceCrossFade.h"
#include "AudioDevice.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SoundNodeDistanceCrossFade)

/*-----------------------------------------------------------------------------
	USoundNodeDistanceCrossFade implementation.
-----------------------------------------------------------------------------*/
USoundNodeDistanceCrossFade::USoundNodeDistanceCrossFade(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

float USoundNodeDistanceCrossFade::GetMaxDistance() const
{
	float MaxDistance = 0.0f;

	for (int32 CrossFadeIndex = 0; CrossFadeIndex < CrossFadeInput.Num(); CrossFadeIndex++)
	{
		float FadeInDistanceMax = CrossFadeInput[CrossFadeIndex].FadeInDistanceEnd;
		float FadeOutDistanceMax = CrossFadeInput[CrossFadeIndex].FadeOutDistanceEnd;

		if (FadeInDistanceMax > MaxDistance)
		{
			MaxDistance = FadeInDistanceMax;
		}

		if (FadeOutDistanceMax > MaxDistance)
		{
			MaxDistance = FadeOutDistanceMax;
		}
	}

	for (USoundNode* ChildNode : ChildNodes)
	{
		if (ChildNode)
		{
			ChildNode->ConditionalPostLoad();
			MaxDistance = FMath::Max(ChildNode->GetMaxDistance(), MaxDistance);
		}
	}
	return MaxDistance;
}


void USoundNodeDistanceCrossFade::ParseNodes( FAudioDevice* AudioDevice, const UPTRINT NodeWaveInstanceHash, FActiveSound& ActiveSound, const FSoundParseParameters& ParseParams, TArray<FWaveInstance*>& WaveInstances )
{
	int32 NumSoundsActive = 0;

	FSoundParseParameters UpdatedParams = ParseParams;
	for( int32 ChildNodeIndex = 0; ChildNodeIndex < ChildNodes.Num(); ChildNodeIndex++ )
	{
		if( ChildNodes[ ChildNodeIndex ] != NULL )
		{
			// get the various distances for this input so we can fade in/out the volume correctly
			const float FadeInDistanceMin = CrossFadeInput[ ChildNodeIndex ].FadeInDistanceStart;
			const float FadeInDistanceMax = CrossFadeInput[ ChildNodeIndex ].FadeInDistanceEnd;

			const float FadeOutDistanceMin = CrossFadeInput[ ChildNodeIndex ].FadeOutDistanceStart;
			const float FadeOutDistanceMax = CrossFadeInput[ ChildNodeIndex ].FadeOutDistanceEnd;

			// watch out here.  If one is playing the sound on the PlayerController then this will not update correctly as PlayerControllers don't move in normal play
			const float Distance = GetCurrentDistance(AudioDevice, ActiveSound, ParseParams);

			// determine the volume amount we should set the component to before "playing"
			float VolumeToSet = 1.0f;
			//UE_LOGF(LogAudio, Log,  "  USoundNodeDistanceCrossFade.  Distance: %f ChildNodeIndex: %d CurrLoc: %ls  ListenerLoc: %ls", Distance, ChildNodeIndex, *AudioComponent->CurrentLocation.ToString(), *AudioComponent->Listener->Location.ToString() );

			// Check whether crossfading is allowed
			if( !AllowCrossfading(ActiveSound) )
			{
				VolumeToSet = CrossFadeInput[ ChildNodeIndex ].Volume;
			}
			else if( ( Distance >= FadeInDistanceMin ) && ( Distance <= FadeInDistanceMax ) )
			{
				VolumeToSet = (FadeInDistanceMax > 0.f ? CrossFadeInput[ ChildNodeIndex ].Volume * ( 0.0f + ( Distance - FadeInDistanceMin ) / ( FadeInDistanceMax - FadeInDistanceMin ) ) : CrossFadeInput[ ChildNodeIndex ].Volume);
				//UE_LOGF(LogAudio, Log,  "     FadeIn.  Distance: %f,  VolumeToSet: %f", Distance, VolumeToSet );
			}
			// else if we are inside the FadeOut edge
			else if( ( Distance >= FadeOutDistanceMin ) && ( Distance <= FadeOutDistanceMax ) )
			{
				VolumeToSet = (FadeOutDistanceMax > 0.f ? CrossFadeInput[ ChildNodeIndex ].Volume * ( 1.0f - ( Distance - FadeOutDistanceMin ) / ( FadeOutDistanceMax - FadeOutDistanceMin ) ) : 0.f);
				//UE_LOGF(LogAudio, Log,  "     FadeOut.  Distance: %f,  VolumeToSet: %f", Distance, VolumeToSet );
			}
			// else we are in between the fading edges of the CrossFaded sound and we should play the
			// sound at the CrossFadeInput's specified volume
			else if( ( Distance >= FadeInDistanceMax ) && ( Distance <= FadeOutDistanceMin ) )
			{
				VolumeToSet = CrossFadeInput[ ChildNodeIndex ].Volume;
				//UE_LOGF(LogAudio, Log,  "     In Between.  Distance: %f,  VolumeToSet: %f", Distance, VolumeToSet );
			}
			// else we are outside of the range of this CrossFadeInput and should not play anything
			else
			{
				//UE_LOGF(LogAudio, Log,  "     OUTSIDE!!!");
				VolumeToSet = 0.f; //CrossFadeInput( ChildNodeIndex ).Volume;
			}

			if (VolumeToSet > 0.0f)
			{
				++NumSoundsActive;
			}

			UpdatedParams.Volume = ParseParams.Volume * VolumeToSet;

			const bool bWasFinished = ActiveSound.bFinished;
			const int32 WaveInstanceCount = WaveInstances.Num();

			// "play" the rest of the tree
			ChildNodes[ ChildNodeIndex ]->ParseNodes( AudioDevice, GetNodeWaveInstanceHash(NodeWaveInstanceHash, ChildNodes[ChildNodeIndex], ChildNodeIndex), ActiveSound, UpdatedParams, WaveInstances );
		
			// Don't let an out of range cross-fade branch keep an active sound alive
			if (bWasFinished && VolumeToSet <= 0.f && WaveInstanceCount == WaveInstances.Num() && !ActiveSound.GetSound()->IsLooping())
			{
				ActiveSound.bFinished = true;
			}
		}
	}

	// Write out the results of the NumSounds count. We can't do this in the loop since the table storing this data will potentially reallocate
	RETRIEVE_SOUNDNODE_PAYLOAD(sizeof(int32));
	DECLARE_SOUNDNODE_ELEMENT(int32, NumSoundsUsedInCrossFade);

	NumSoundsUsedInCrossFade = NumSoundsActive;
}

int32 USoundNodeDistanceCrossFade::GetNumSounds(const UPTRINT NodeWaveInstanceHash, FActiveSound& ActiveSound) const
{
	RETRIEVE_SOUNDNODE_PAYLOAD(sizeof(int32));
	DECLARE_SOUNDNODE_ELEMENT(int32, NumSoundsUsedInCrossFade);

	return NumSoundsUsedInCrossFade;
}


void USoundNodeDistanceCrossFade::CreateStartingConnectors()
{
	// Mixers default with two connectors.
	InsertChildNode( ChildNodes.Num() );
	InsertChildNode( ChildNodes.Num() );
}


void USoundNodeDistanceCrossFade::InsertChildNode( int32 Index )
{
	Super::InsertChildNode( Index );

	if (Index >= CrossFadeInput.Num())
	{
		CrossFadeInput.InsertZeroed(Index);
		CrossFadeInput[Index].Volume = 1.0f;
	}
}


void USoundNodeDistanceCrossFade::RemoveChildNode( int32 Index )
{
	Super::RemoveChildNode( Index );
	CrossFadeInput.RemoveAt( Index );
}

#if WITH_EDITOR
void USoundNodeDistanceCrossFade::SetChildNodes(TArray<USoundNode*>& InChildNodes)
{
	Super::SetChildNodes(InChildNodes);

	if (CrossFadeInput.Num() < ChildNodes.Num())
	{
		const int32 OldSize = CrossFadeInput.Num();
		const int32 NumToAdd = ChildNodes.Num() - OldSize;
		CrossFadeInput.AddZeroed(NumToAdd);
		for (int32 NewIndex = OldSize; NewIndex < CrossFadeInput.Num(); ++NewIndex)
		{
			CrossFadeInput[NewIndex].Volume = 1.0f;
		}
	}
	else if (CrossFadeInput.Num() > ChildNodes.Num())
	{
		const int32 NumToRemove = CrossFadeInput.Num() - ChildNodes.Num();
		CrossFadeInput.RemoveAt(CrossFadeInput.Num() - NumToRemove, NumToRemove);
	}
}
#endif //WITH_EDITOR

float USoundNodeDistanceCrossFade::GetCurrentDistance(FAudioDevice* AudioDevice, FActiveSound& ActiveSound, const FSoundParseParameters& ParseParams) const
{
	if (!ActiveSound.bLocationDefined)
	{
		return 0.f;
	}

	check(AudioDevice);
	FVector ListenerPosition;
	const bool bAllowOverride = true;
	AudioDevice->GetListenerPosition(ActiveSound.GetClosestListenerIndex(), ListenerPosition, bAllowOverride);
	return FVector::Dist(ParseParams.Transform.GetTranslation(), ListenerPosition);
}

bool USoundNodeDistanceCrossFade::AllowCrossfading(FActiveSound& ActiveSound) const
{
	// Ignore distance calculations for preview components as they are undefined
	return ActiveSound.bLocationDefined;
}

