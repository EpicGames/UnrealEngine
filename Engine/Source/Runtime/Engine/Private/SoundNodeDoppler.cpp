// Copyright Epic Games, Inc. All Rights Reserved.


#include "Sound/SoundNodeDoppler.h"
#include "AudioDevice.h"
#include "Kismet/KismetMathLibrary.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SoundNodeDoppler)

/*-----------------------------------------------------------------------------
         USoundNodeDoppler implementation.
-----------------------------------------------------------------------------*/
USoundNodeDoppler::USoundNodeDoppler(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	DopplerIntensity = 1.0f;
	bUseSmoothing = false;
	SmoothingInterpSpeed = 5.0f;
}

void USoundNodeDoppler::ParseNodes( FAudioDevice* AudioDevice, const UPTRINT NodeWaveInstanceHash, FActiveSound& ActiveSound, const FSoundParseParameters& ParseParams, TArray<FWaveInstance*>& WaveInstances )
{
	RETRIEVE_SOUNDNODE_PAYLOAD(sizeof(float));
	DECLARE_SOUNDNODE_ELEMENT(float, CurrentPitchScale);

	FSoundParseParameters UpdatedParams = ParseParams;

	check(AudioDevice);
	const TArray<FListener>& Listeners = AudioDevice->GetListeners();

	int32 ListenerIndex = ActiveSound.GetClosestListenerIndex();
	if (ListenerIndex >= Listeners.Num())
	{
		ListenerIndex = 0;
	}

	const FListener& Listener = Listeners[ListenerIndex];

	// Default the parse to using the setting for smoothing
	if (*RequiresInitialization)
	{
		*RequiresInitialization = 0;

		// First time, do no smoothing, but initialize the current pitch scale value to the first value returned from this function
		CurrentPitchScale = GetDopplerPitchMultiplier(CurrentPitchScale, false, Listener, ParseParams.Transform.GetTranslation(), ParseParams.Velocity, AudioDevice->GetDeviceDeltaTime());
		UpdatedParams.Pitch *= CurrentPitchScale;
	}
	else
	{
		// Subsequent calls to this will do smoothing from the first initial value
		UpdatedParams.Pitch *= GetDopplerPitchMultiplier(CurrentPitchScale, bUseSmoothing, Listener, ParseParams.Transform.GetTranslation(), ParseParams.Velocity, AudioDevice->GetDeviceDeltaTime());
	}

	Super::ParseNodes(AudioDevice, NodeWaveInstanceHash, ActiveSound, UpdatedParams, WaveInstances);
}

float USoundNodeDoppler::GetDopplerPitchMultiplier(float& CurrentPitchScale, bool bSmooth, FListener const& InListener, const FVector Location, const FVector Velocity, float DeltaTime)
{
	static const float SpeedOfSoundInAirAtSeaLevel = 33000.f;		// cm/sec

	FVector const SourceToListenerNorm = (InListener.Transform.GetTranslation() - Location).GetSafeNormal();

	// find source and listener speeds along the line between them
	float const SourceVelMagTorwardListener = Velocity | SourceToListenerNorm;
	float const ListenerVelMagAwayFromSource = InListener.Velocity | SourceToListenerNorm;

	// multiplier = 1 / (1 - ((sourcevel - listenervel) / speedofsound) );
	float const InvDopplerPitchScale = 1.f - ( (SourceVelMagTorwardListener - ListenerVelMagAwayFromSource) / SpeedOfSoundInAirAtSeaLevel );
	float const PitchScale = 1.f / InvDopplerPitchScale;

	// factor in user-specified intensity
	float const FinalPitchScale = ((PitchScale - 1.f) * DopplerIntensity) + 1.f;

	//UE_LOGF(LogAudio, Log, "Applying doppler pitchscale %f, raw scale %f, deltaspeed was %f", FinalPitchScale, PitchScale, ListenerVelMagAwayFromSource - SourceVelMagTorwardListener);

	if (bSmooth)
	{
		CurrentPitchScale = UKismetMathLibrary::FInterpTo(CurrentPitchScale, FinalPitchScale, DeltaTime, SmoothingInterpSpeed);

		return CurrentPitchScale;
	}
	else
	{
		return FinalPitchScale;
	}
}

