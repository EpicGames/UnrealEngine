// Copyright Epic Games, Inc. All Rights Reserved.


#include "Sound/SoundNodeEnveloper.h"
#include "ActiveSound.h"
#include "Distributions/DistributionFloatConstantCurve.h"
#include "Math/InterpCurve.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SoundNodeEnveloper)

/*-----------------------------------------------------------------------------
USoundNodeEnveloper implementation.
-----------------------------------------------------------------------------*/
USoundNodeEnveloper::USoundNodeEnveloper(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	PitchMin = 1.0f;
	PitchMax = 1.0f;
	VolumeMin = 1.0f;
	VolumeMax = 1.0f;
}

void USoundNodeEnveloper::PostInitProperties()
{
	Super::PostInitProperties();
	if (!HasAnyFlags(RF_ClassDefaultObject | RF_NeedLoad))
	{
		VolumeCurve.EditorCurveData.AddKey(0.0f, 1.0f);
		PitchCurve.EditorCurveData.AddKey(0.0f, 1.0f);
	}
}

void USoundNodeEnveloper::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);
	if (Ar.IsLoading())
	{
		if (Ar.UEVer() < VER_UE4_SOUND_NODE_ENVELOPER_CURVE_CHANGE)
		{
			if (VolumeInterpCurve_DEPRECATED)
			{
				VolumeCurve.EditorCurveData.Reset();
				for (int32 Index = 0; Index < VolumeInterpCurve_DEPRECATED->ConstantCurve.Points.Num(); ++Index)
				{
					FInterpCurvePoint<float>& Point = VolumeInterpCurve_DEPRECATED->ConstantCurve.Points[Index];
					VolumeCurve.EditorCurveData.AddKey(Point.InVal, Point.OutVal);
				}
			}
			if (PitchInterpCurve_DEPRECATED)
			{
				PitchCurve.EditorCurveData.Reset();
				for (int32 Index = 0; Index < PitchInterpCurve_DEPRECATED->ConstantCurve.Points.Num(); ++Index)
				{
					FInterpCurvePoint<float>& Point = PitchInterpCurve_DEPRECATED->ConstantCurve.Points[Index];
					PitchCurve.EditorCurveData.AddKey(Point.InVal, Point.OutVal);
				}
			}
		}
	}
}

void USoundNodeEnveloper::ParseNodes( FAudioDevice* AudioDevice, const UPTRINT NodeWaveInstanceHash, FActiveSound& ActiveSound, const FSoundParseParameters& ParseParams, TArray<FWaveInstance*>& WaveInstances )
{
	RETRIEVE_SOUNDNODE_PAYLOAD( sizeof( float ) + sizeof(float) + sizeof(float) + sizeof(int32) );
	DECLARE_SOUNDNODE_ELEMENT( float, StartTime );
	DECLARE_SOUNDNODE_ELEMENT( float, UsedVolumeModulation );
	DECLARE_SOUNDNODE_ELEMENT( float, UsedPitchModulation );
	DECLARE_SOUNDNODE_ELEMENT( int32, LastLoopCount );

	if( *RequiresInitialization )
	{
		StartTime = ActiveSound.PlaybackTimeNonVirtualized - ParseParams.StartTime;
		UsedVolumeModulation = VolumeMax + ( ( VolumeMin - VolumeMax ) * RandomStream.FRand() );
		UsedPitchModulation = PitchMax + ( ( PitchMin - PitchMax ) * RandomStream.FRand() );
		LastLoopCount = -1;

		*RequiresInitialization = false;
	}

	float PlayTime = ActiveSound.PlaybackTimeNonVirtualized - StartTime;

	if(bLoop && PlayTime > LoopEnd)
	{
		if( PlayTime > GetDuration() )
		{
			return;
		}

		float LoopDuration = LoopEnd - LoopStart;
		int32 CurrentLoopCount = (int32)(PlayTime - LoopStart) / FMath::Max(UE_SMALL_NUMBER, LoopDuration);
		PlayTime -= CurrentLoopCount * LoopDuration;

		if( CurrentLoopCount == LoopCount && !bLoopIndefinitely && LoopCount != 0 )
		{
			PlayTime += LoopDuration;
		}
		else if ( CurrentLoopCount != LastLoopCount )
		{
			// randomize multipliers for the new repeat
			UsedVolumeModulation = VolumeMax + ( ( VolumeMin - VolumeMax ) * RandomStream.FRand() );
			UsedPitchModulation = PitchMax + ( ( PitchMin - PitchMax ) * RandomStream.FRand() );
			LastLoopCount = CurrentLoopCount;
		}
	}

	FSoundParseParameters UpdatedParams = ParseParams;

	UpdatedParams.Volume *= VolumeCurve.GetRichCurve()->Eval(PlayTime) * UsedVolumeModulation;

	UpdatedParams.Pitch *= PitchCurve.GetRichCurve()->Eval(PlayTime) * UsedPitchModulation;

	Super::ParseNodes(AudioDevice, NodeWaveInstanceHash, ActiveSound, UpdatedParams, WaveInstances);
}

float USoundNodeEnveloper::GetDuration()
{
	float ChildDuration = (ChildNodes.Num() > 0 && ChildNodes[0] != NULL) ? ChildNodes[0]->GetDuration() : 0;

	if (bLoop)
	{
		if (bLoopIndefinitely)
		{
			return INDEFINITELY_LOOPING_DURATION;
		}
		else
		{
			return LoopStart + LoopCount*(LoopEnd-LoopStart) + DurationAfterLoop;
		}
	}
	else
	{
		return ChildDuration;
	}
}

#if WITH_EDITOR
void USoundNodeEnveloper::PostEditChangeProperty( struct FPropertyChangedEvent& PropertyChangedEvent)
{
	// a few sanity checks
	if( LoopCount < 0 )
	{
		LoopCount = 0;
	}

	if( LoopEnd < LoopStart )
	{
		LoopEnd = LoopStart;
	}

	if( DurationAfterLoop < 0 )
	{
		DurationAfterLoop = 0;
	}
}
#endif // WITH_EDITOR

