// Copyright Epic Games, Inc. All Rights Reserved.


#include "Sound/SoundNodeGroupControl.h"
#include "ActiveSound.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SoundNodeGroupControl)

DEFINE_LOG_CATEGORY_STATIC(LogSoundNodeGroupControl, Log, All);

TMap< USoundNodeGroupControl*, TArray< TMap< FActiveSound*, int32> > > USoundNodeGroupControl::GroupControlSlotUsageMap;

/*-----------------------------------------------------------------------------
    USoundNodeGroupControl implementation.
-----------------------------------------------------------------------------*/
USoundNodeGroupControl::USoundNodeGroupControl(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

void USoundNodeGroupControl::FixGroupSizesArray()
{
	const int32 CorrectGroupSizesCount = FMath::Max(ChildNodes.Num() - 1, 0);

	// If group sizes and children got out of sync, we fix it first.
	if( GroupSizes.Num() < CorrectGroupSizesCount )
	{
		GroupSizes.AddZeroed( CorrectGroupSizesCount - GroupSizes.Num() );
	}
	else if( GroupSizes.Num() > CorrectGroupSizesCount )
	{
		const int32 NumToRemove = GroupSizes.Num() - CorrectGroupSizesCount;
		GroupSizes.RemoveAt( GroupSizes.Num() - NumToRemove, NumToRemove );
	}
}

bool USoundNodeGroupControl::NotifyWaveInstanceFinished( FWaveInstance* WaveInstance )
{
	FActiveSound& ActiveSound = *WaveInstance->ActiveSound;
	const UPTRINT NodeWaveInstanceHash = WaveInstance->NotifyBufferFinishedHooks.GetHashForNode(this);
	RETRIEVE_SOUNDNODE_PAYLOAD( sizeof( int32 ) );
	DECLARE_SOUNDNODE_ELEMENT( int32, NodeIndex );
	check( *RequiresInitialization == 0 );

	TArray<TMap<FActiveSound*, int32> >* Slots = GroupControlSlotUsageMap.Find(this);
	if (Slots == NULL)
	{
		return false;
	}

	TMap<FActiveSound*, int32>& ActiveSounds = (*Slots)[NodeIndex];
	int32* ActiveSoundWaveInstanceCount = ActiveSounds.Find(WaveInstance->ActiveSound);

	if (ActiveSoundWaveInstanceCount == NULL)
	{
		return false;
	}
	
	if (*ActiveSoundWaveInstanceCount > 1)
	{
		--(*ActiveSoundWaveInstanceCount);
	}
	else if (ActiveSounds.Num() > 1)
	{
		ActiveSounds.Remove(WaveInstance->ActiveSound);
	}
	else
	{
		int32 ActiveSoundCount = 0;
		for (int32 SlotIndex = 0; SlotIndex < (*Slots).Num(); ++SlotIndex)
		{
			ActiveSoundCount += (*Slots)[SlotIndex].Num();
		}
		if (ActiveSoundCount == 1)
		{
			GroupControlSlotUsageMap.Remove(this);
		}
		else
		{
			ActiveSounds.Remove(WaveInstance->ActiveSound);
		}
	}

	return false;
}

void USoundNodeGroupControl::ParseNodes( FAudioDevice* AudioDevice, const UPTRINT NodeWaveInstanceHash, FActiveSound& ActiveSound, const FSoundParseParameters& ParseParams, TArray<FWaveInstance*>& WaveInstances )
{
	bool bInitializing = false;
	const int32 WaveInstanceCount = WaveInstances.Num();
	TArray<TMap<FActiveSound*, int32> >* Slots = NULL;
	int32 NodeIndexCopy = 0; // Copy of NodeIndex taken before parse wrecks it

	{
		RETRIEVE_SOUNDNODE_PAYLOAD( sizeof( int32 ) );
		DECLARE_SOUNDNODE_ELEMENT( int32, NodeIndex );

		bInitializing = (*RequiresInitialization != 0);

		if( bInitializing )
		{
			NodeIndex = 0;
			Slots = GroupControlSlotUsageMap.Find(this);

			if (Slots)
			{
				for(; NodeIndex < GroupSizes.Num(); ++NodeIndex)
				{
					if ((*Slots)[NodeIndex].Num() < GroupSizes[NodeIndex])
					{
						break;
					}
				}
			}

			*RequiresInitialization = false;
		}
		NodeIndexCopy = NodeIndex;

		// "play" the sound node that was selected
		if( NodeIndex < ChildNodes.Num() && ChildNodes[ NodeIndex ] )
		{
			FSoundParseParameters UpdatedParams = ParseParams;
			UpdatedParams.NotifyBufferFinishedHooks.AddNotify(this, NodeWaveInstanceHash);
			ChildNodes[ NodeIndex ]->ParseNodes( AudioDevice, GetNodeWaveInstanceHash(NodeWaveInstanceHash, ChildNodes[NodeIndex], NodeIndex), ActiveSound, UpdatedParams, WaveInstances );
		}
	}

	const int32 WavesAdded = WaveInstances.Num() - WaveInstanceCount;

	// If no sounds ended up being generated as a child of this then discard the chosen
	// NodeIndex and pick again when we will actually play something
	if (bInitializing)
	{
		// We must retrieve the NodeIndex again as during ParseNodes it could have been resized 
		// making previous references invalid
		RETRIEVE_SOUNDNODE_PAYLOAD( sizeof( int32 ) );

		if (WavesAdded == 0)
		{
			*RequiresInitialization = true;
		}
		else 
		{
			if (Slots == NULL)
			{
				Slots = &GroupControlSlotUsageMap.Add(this);
				Slots->SetNum(GroupSizes.Num() + 1);
			}
			(*Slots)[NodeIndexCopy].Add(&ActiveSound, WavesAdded);
		}
	}
	else if (WavesAdded > 0)
	{
		if (Slots == NULL)
		{
			Slots = &GroupControlSlotUsageMap.Add(this);
			Slots->SetNum(GroupSizes.Num() + 1);
		}
		(*Slots)[NodeIndexCopy].FindOrAdd(&ActiveSound) = WavesAdded;
	}
}

void USoundNodeGroupControl::CreateStartingConnectors()
{
	InsertChildNode( ChildNodes.Num() );
	InsertChildNode( ChildNodes.Num() );
}

void USoundNodeGroupControl::InsertChildNode(int32 Index)
{
	FixGroupSizesArray();

	check( Index >= 0 && Index <= GroupSizes.Num() + 1 );

	if (Index >= GroupSizes.Num())
	{
		GroupSizes.Add(1);
	}
	else
	{
		GroupSizes.Insert(1, Index);
	}

	Super::InsertChildNode(Index);
}

void USoundNodeGroupControl::RemoveChildNode(int32 Index)
{
	FixGroupSizesArray();

	check( Index >= 0 && Index <= GroupSizes.Num() );

	if (GroupSizes.Num() > 0)
	{
		if (Index == GroupSizes.Num())
		{
			GroupSizes.RemoveAt(Index - 1);
		}
		else
		{
			GroupSizes.RemoveAt(Index);
		}
	}

	Super::RemoveChildNode(Index);
}

#if WITH_EDITOR
void USoundNodeGroupControl::SetChildNodes(TArray<USoundNode*>& InChildNodes)
{
	Super::SetChildNodes(InChildNodes);

	const int32 CorrectGroupSizesCount = FMath::Max(ChildNodes.Num() - 1, 0);

	if (GroupSizes.Num() < CorrectGroupSizesCount)
	{
		while (GroupSizes.Num() < CorrectGroupSizesCount)
		{
			GroupSizes.Add(1);
		}
	}
	else if (GroupSizes.Num() > CorrectGroupSizesCount)
	{
		const int32 NumToRemove = GroupSizes.Num() - CorrectGroupSizesCount;
		GroupSizes.RemoveAt(GroupSizes.Num() - NumToRemove, NumToRemove);
	}
}
#endif //WITH_EDITOR

