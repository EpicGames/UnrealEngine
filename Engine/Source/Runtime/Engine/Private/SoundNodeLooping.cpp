// Copyright Epic Games, Inc. All Rights Reserved.


#include "Sound/SoundNodeLooping.h"
#include "ActiveSound.h"
#include "Sound/SoundNodeWavePlayer.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SoundNodeLooping)

/*-----------------------------------------------------------------------------
	USoundNodeLooping implementation.
-----------------------------------------------------------------------------*/
USoundNodeLooping::USoundNodeLooping(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, LoopCount(1)
	, bLoopIndefinitely(true)
{
}

void USoundNodeLooping::ParseNodes( FAudioDevice* AudioDevice, const UPTRINT NodeWaveInstanceHash, FActiveSound& ActiveSound, const FSoundParseParameters& ParseParams, TArray<FWaveInstance*>& WaveInstances )
{
	{
		RETRIEVE_SOUNDNODE_PAYLOAD(sizeof(int32));
		DECLARE_SOUNDNODE_ELEMENT(int32, CurrentLoopCount);

		if (*RequiresInitialization)
		{
			CurrentLoopCount = 0;

			*RequiresInitialization = false;
		}
	}

#if !(NO_LOGGING || UE_BUILD_SHIPPING || UE_BUILD_TEST)
	if (bLoopIndefinitely && !ActiveSound.bWarnedAboutOrphanedLooping && ActiveSound.GetAudioComponentID() == 0 && ActiveSound.FadeOut == FActiveSound::EFadeOut::None)
	{
		UE_LOGF(LogAudio, Warning, "Detected orphaned looping sound '%ls'.", *ActiveSound.GetSound()->GetName());
		ActiveSound.bWarnedAboutOrphanedLooping = true;
	}
#endif

	// Do not continue parsing if virtualized and this is functioning as a one-shot
	if (!bLoopIndefinitely && ActiveSound.bHasVirtualized)
	{
		return;
	}

	FSoundParseParameters UpdatedParams = ParseParams;
	UpdatedParams.NotifyBufferFinishedHooks.AddNotify(this, NodeWaveInstanceHash);
	UpdatedParams.bEnableRetrigger = true;
	Super::ParseNodes( AudioDevice, NodeWaveInstanceHash, ActiveSound, UpdatedParams, WaveInstances );

	if (ActiveSound.bFinished)
	{
		RETRIEVE_SOUNDNODE_PAYLOAD(sizeof(int32));
		DECLARE_SOUNDNODE_ELEMENT(int32, CurrentLoopCount);
		
		if (bLoopIndefinitely || CurrentLoopCount < LoopCount)
		{
			// We did not find a sound to play in our children but we are set to looping.
			// Reset children to allow random nodes to reinitialize on our next attempt.
			ResetChildren(NodeWaveInstanceHash, ActiveSound, CurrentLoopCount);
			ActiveSound.bFinished = false;
		}
	}
}

bool USoundNodeLooping::NotifyWaveInstanceFinished( FWaveInstance* InWaveInstance )
{
	FActiveSound& ActiveSound = *InWaveInstance->ActiveSound;
	const UPTRINT NodeWaveInstanceHash = InWaveInstance->NotifyBufferFinishedHooks.GetHashForNode(this);
	RETRIEVE_SOUNDNODE_PAYLOAD(sizeof(int32));
	DECLARE_SOUNDNODE_ELEMENT(int32, CurrentLoopCount);

	const bool bRequiresInitialization = (*RequiresInitialization) != 0;

	if (!bRequiresInitialization)
	{
		if (bLoopIndefinitely || ++CurrentLoopCount < LoopCount)
		{
			ResetChildren(NodeWaveInstanceHash, ActiveSound, CurrentLoopCount);

			// Reset wave instances that notified us of completion.
			InWaveInstance->bIsStarted = false;
			InWaveInstance->bIsFinished = false;
			return true;
		}
	}

	return false;
}


void USoundNodeLooping::ResetChildren(const UPTRINT NodeWaveInstanceHash, FActiveSound& ActiveSound, int32& CurrentLoopCount)
{
	struct FNodeHashPairs
	{
		USoundNode* Node;
		UPTRINT NodeWaveInstanceHash;

		FNodeHashPairs(USoundNode* InNode, const UPTRINT InHash)
			: Node(InNode)
			, NodeWaveInstanceHash(InHash)
		{
		}
	};

	TArray<FNodeHashPairs> NodesToReset;

	for (int32 ChildNodeIndex = 0; ChildNodeIndex < ChildNodes.Num(); ++ChildNodeIndex)
	{
		USoundNode* ChildNode = ChildNodes[ChildNodeIndex];
		if (ChildNode)
		{
			NodesToReset.Add(FNodeHashPairs(ChildNode, GetNodeWaveInstanceHash(NodeWaveInstanceHash, ChildNode, ChildNodeIndex)));
		}
	}

	for (int32 ResetNodeIndex = 0; ResetNodeIndex < NodesToReset.Num(); ++ResetNodeIndex)
	{
		// cache both fields, ref to NodesToReset[ResetNodeIndex] can become invalid after NodesToReset.Add due to memory reallocation
		const UPTRINT ResetNodeWaveInstanceHash = NodesToReset[ResetNodeIndex].NodeWaveInstanceHash;
		USoundNode* ResetNode = NodesToReset[ResetNodeIndex].Node;

		// Reset all child nodes so they are initialized again.
		uint32* Offset = ActiveSound.SoundNodeOffsetMap.Find(ResetNodeWaveInstanceHash);
		if (Offset)
		{
			bool* bRequiresInitialization = (bool*)&ActiveSound.SoundNodeData[*Offset];
			*bRequiresInitialization = true;
		}

		if (ResetNode->ChildNodes.Num())
		{
			for (int32 ResetChildIndex = 0; ResetChildIndex < ResetNode->ChildNodes.Num(); ++ResetChildIndex)
			{
				USoundNode* ResetChildNode = ResetNode->ChildNodes[ResetChildIndex];
				if (ResetChildNode)
				{
					NodesToReset.Add(FNodeHashPairs(ResetChildNode, GetNodeWaveInstanceHash(ResetNodeWaveInstanceHash, ResetChildNode, ResetChildIndex)));
				}
			}
		}
		else if (ResetNode->IsA<USoundNodeWavePlayer>())
		{
			FWaveInstance* WaveInstance = ActiveSound.FindWaveInstance(ResetNodeWaveInstanceHash);
			if (WaveInstance)
			{
				WaveInstance->bAlreadyNotifiedHook = true;
				WaveInstance->bIsStarted = false;
				WaveInstance->bIsFinished = false;
			}
		}
	}
}

float USoundNodeLooping::GetDuration()
{
	// Assume no duration (i.e. no input node)
	float Duration = 0.0f;

	// If we have any child nodes
	if (ChildNodes.Num() > 0)
	{
		// If we're told to loop indefinitely, then the duration will be "infinite"
		if (bLoopIndefinitely)
		{
			Duration = INDEFINITELY_LOOPING_DURATION;
		}
		else
		{
			// Looping nodes can only have one child node
			check(ChildNodes.Num() == 1);
			if (USoundNode* Child = ChildNodes[0])
			{
				// Duration will be the loop count times the child node duration
				Duration = LoopCount * Child->GetDuration();
			}
		}
	}
	
	return Duration;
}

int32 USoundNodeLooping::GetNumSounds(const UPTRINT NodeWaveInstanceHash, FActiveSound& ActiveSound) const
{
	// Number of sounds this node plays is essentially infinite if told to bLoopIndefinitely
	if (bLoopIndefinitely)
	{
		return (int32)INDEFINITELY_LOOPING_DURATION;
	}
	// Looping nodes count as 1 sound finishing since the looping node captures 
	// sounddone hooks except for the last one (when the loop count is reached)
	return 1;
}




