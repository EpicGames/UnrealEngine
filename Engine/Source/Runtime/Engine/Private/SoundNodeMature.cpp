// Copyright Epic Games, Inc. All Rights Reserved.


#include "Sound/SoundNodeMature.h"
#include "Engine/Engine.h"
#include "ActiveSound.h"
#include "Sound/SoundNodeWavePlayer.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SoundNodeMature)

/*-----------------------------------------------------------------------------
       USoundNodeMature implementation.
-----------------------------------------------------------------------------*/
USoundNodeMature::USoundNodeMature(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

int32 USoundNodeMature::GetMaxChildNodes() const
{
	return MAX_ALLOWED_CHILD_NODES;
}

enum EMaturityChildType
{
	ChildType_None		= 0,
	ChildType_Mature	= 1,
	ChildType_NonMature	= 2,
};

/**
 * Recursively traverses the sound nodes until it finds a sound wave to evaluate its maturity level.
 *
 * @param	Node	The sound node to start iterating from.
 *
 * @return	ChildType_Mature if the first sound wave is a mature sound;
 *			ChildType_NonMature if it is not mature;
 * 			ChildType_None if no sound wave was found.
 */
EMaturityChildType GetMaturityTypeForChild( USoundNode* Node )
{
	EMaturityChildType Type = ChildType_None;

	if( Node )
	{
		// Try to see if the given node is a sound wave node.
		USoundNodeWavePlayer* WavePlayer = Cast<USoundNodeWavePlayer>( Node );
		if( WavePlayer && WavePlayer->GetSoundWave() )
		{
			if( WavePlayer->GetSoundWave()->bMature )
			{
				Type = ChildType_Mature;
			}
			else
			{
				Type = ChildType_NonMature;
			}
		}
		// Not a sound wave node; try to find one in the child nodes.
		else
		{
			// Iterate through all child nodes, looking for sound wave nodes.
			for( int32 ChildIndex = 0; Type == ChildType_None && ChildIndex < Node->ChildNodes.Num() ; ++ChildIndex )
			{
				Type = GetMaturityTypeForChild(Node->ChildNodes[ChildIndex]);
			}
		}
	}

	return Type;
}

void USoundNodeMature::ParseNodes( FAudioDevice* AudioDevice, const UPTRINT NodeWaveInstanceHash, FActiveSound& ActiveSound, const FSoundParseParameters& ParseParams, TArray<FWaveInstance*>& WaveInstances )
{
	RETRIEVE_SOUNDNODE_PAYLOAD( sizeof( int32 ) );
	DECLARE_SOUNDNODE_ELEMENT( int32, NodeIndex );

	// Pick a random child node and save the index.
	if( *RequiresInitialization )
	{
		*RequiresInitialization = 0;

		// Make a list of mature and non-mature child nodes.
		TArray<int32> MatureChildNodes;
		MatureChildNodes.Empty( ChildNodes.Num() );

		TArray<int32> NonMatureChildNodes;
		NonMatureChildNodes.Empty( ChildNodes.Num() );

		for( int32 i = 0; i < ChildNodes.Num() ; ++i )
		{
			if( ChildNodes[ i ] )
			{
				EMaturityChildType Type = GetMaturityTypeForChild( ChildNodes[ i ] );

				if( ChildType_Mature == Type )
				{
					MatureChildNodes.Add( i );
				}
				else if( ChildType_NonMature == Type )
				{
					NonMatureChildNodes.Add( i );
				}
				else
				{
					UE_LOGF(LogAudio, Warning, "SoundNodeMature(%ls) has a child which is not eventually linked to a sound node wave", *GetPathName() );
				}
			}
		}

		// Select a child node.
		NodeIndex = -1;
		if( GEngine->bAllowMatureLanguage )
		{
			// If mature language is allowed, prefer a mature node.
			if( MatureChildNodes.Num() > 0 )
			{
				NodeIndex = MatureChildNodes[ 0 ];
			}
			else if( NonMatureChildNodes.Num() > 0 )
			{
				NodeIndex = NonMatureChildNodes[ 0 ];
			}
		}
		else
		{
			// If mature language is not allowed, prefer a non-mature node.
			if( NonMatureChildNodes.Num() > 0 )
			{
				NodeIndex = NonMatureChildNodes[ 0 ];
			}
			else
			{
				UE_LOGF(LogAudio, Warning, "SoundNodeMature(%ls): GEngine->bAllowMatureLanguage is false, no non-mature child sound exists", *GetPathName() );
			}
		}
	}

	// "play" the sound node that was selected
	if( NodeIndex >= 0 && NodeIndex < ChildNodes.Num() && ChildNodes[ NodeIndex ] )
	{
		ChildNodes[ NodeIndex ]->ParseNodes( AudioDevice, GetNodeWaveInstanceHash(NodeWaveInstanceHash, ChildNodes[NodeIndex], NodeIndex), ActiveSound, ParseParams, WaveInstances );
	}
}

void USoundNodeMature::CreateStartingConnectors()
{
	InsertChildNode( ChildNodes.Num() );
	InsertChildNode( ChildNodes.Num() );
}

void USoundNodeMature::PostLoad()
{
	Super::PostLoad();

	if( !GIsEditor && GEngine && !IsRooted() && ChildNodes.Num() >= 2 )
	{
		// Make sure the SoundCue has gotten all the SoundWavePlayers in to memory
		GetOuter()->ConditionalPostLoad();
		for( int32 i = ChildNodes.Num() - 1; i >= 0 ; --i )
		{
			USoundNodeWavePlayer *WavePlayer = Cast<USoundNodeWavePlayer>(ChildNodes[i]);
			if (WavePlayer && WavePlayer->GetSoundWave() && WavePlayer->GetSoundWave()->bMature != GEngine->bAllowMatureLanguage)
			{
				ChildNodes.RemoveAt(i);
			}
		}
	}
}

