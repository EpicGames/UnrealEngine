// Copyright Epic Games, Inc. All Rights Reserved.


#include "Sound/SoundNodeMixer.h"
#include "ActiveSound.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SoundNodeMixer)

/*-----------------------------------------------------------------------------
	USoundNodeMixer implementation.
-----------------------------------------------------------------------------*/
USoundNodeMixer::USoundNodeMixer(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

void USoundNodeMixer::ParseNodes( FAudioDevice* AudioDevice, const UPTRINT NodeWaveInstanceHash, FActiveSound& ActiveSound, const FSoundParseParameters& ParseParams, TArray<FWaveInstance*>& WaveInstances )
{
	FSoundParseParameters UpdatedParams = ParseParams;
	for( int32 ChildNodeIndex = 0; ChildNodeIndex < ChildNodes.Num(); ++ChildNodeIndex )
	{
		if( ChildNodes[ ChildNodeIndex ] )
		{
			UpdatedParams.Volume = ParseParams.Volume * InputVolume[ ChildNodeIndex ];

			ChildNodes[ ChildNodeIndex ]->ParseNodes( AudioDevice, GetNodeWaveInstanceHash(NodeWaveInstanceHash, ChildNodes[ChildNodeIndex], ChildNodeIndex), ActiveSound, UpdatedParams, WaveInstances );
		}
	}
}

void USoundNodeMixer::CreateStartingConnectors()
{
	// Mixers default with two connectors.
	InsertChildNode( ChildNodes.Num() );
	InsertChildNode( ChildNodes.Num() );
}

void USoundNodeMixer::InsertChildNode( int32 Index )
{
	Super::InsertChildNode( Index );
	InputVolume.InsertUninitialized( Index );
	InputVolume[ Index ] = 1.0f;
}

void USoundNodeMixer::RemoveChildNode( int32 Index )
{
	Super::RemoveChildNode( Index );
	InputVolume.RemoveAt( Index );
}

#if WITH_EDITOR
void USoundNodeMixer::SetChildNodes(TArray<USoundNode*>& InChildNodes)
{
	Super::SetChildNodes(InChildNodes);

	if (InputVolume.Num() < ChildNodes.Num())
	{
		while (InputVolume.Num() < ChildNodes.Num())
		{
			int32 NewIndex = InputVolume.Num();
			InputVolume.InsertUninitialized( NewIndex );
			InputVolume[ NewIndex ] = 1.0f;
		}
	}
	else if (InputVolume.Num() > ChildNodes.Num())
	{
		const int32 NumToRemove = InputVolume.Num() - ChildNodes.Num();
		InputVolume.RemoveAt(InputVolume.Num() - NumToRemove, NumToRemove);
	}
}
#endif //WITH_EDITOR

