// Copyright Epic Games, Inc. All Rights Reserved.


#include "Sound/SoundNodeModulator.h"
#include "ActiveSound.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SoundNodeModulator)

/*-----------------------------------------------------------------------------
	USoundNodeModulator implementation.
-----------------------------------------------------------------------------*/
USoundNodeModulator::USoundNodeModulator(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	PitchMin = 0.95f;
	PitchMax = 1.05f;
	VolumeMin = 0.95f;
	VolumeMax = 1.05f;
}

void USoundNodeModulator::ParseNodes( FAudioDevice* AudioDevice, const UPTRINT NodeWaveInstanceHash, FActiveSound& ActiveSound, const FSoundParseParameters& ParseParams, TArray<FWaveInstance*>& WaveInstances )
{
	RETRIEVE_SOUNDNODE_PAYLOAD( sizeof( float ) + sizeof( float ) );
	DECLARE_SOUNDNODE_ELEMENT( float, UsedVolumeModulation );
	DECLARE_SOUNDNODE_ELEMENT( float, UsedPitchModulation );

	if( *RequiresInitialization )
	{
		UsedVolumeModulation = VolumeMax + ( ( VolumeMin - VolumeMax ) * RandomStream.FRand() );
		UsedPitchModulation = PitchMax + ( ( PitchMin - PitchMax ) * RandomStream.FRand() );

		*RequiresInitialization = 0;
	}

	FSoundParseParameters UpdatedParams = ParseParams;
	UpdatedParams.Volume *= UsedVolumeModulation;
	UpdatedParams.Pitch *= UsedPitchModulation;

	Super::ParseNodes( AudioDevice, NodeWaveInstanceHash, ActiveSound, UpdatedParams, WaveInstances );
}

