// Copyright Epic Games, Inc. All Rights Reserved.


#include "Sound/SoundNodeModulatorContinuous.h"
#include "ActiveSound.h"
#include "IAudioParameterTransmitter.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SoundNodeModulatorContinuous)

float FModulatorContinuousParams::GetValue(const FActiveSound& ActiveSound) const
{
	FAudioParameter Param;
	if (!ActiveSound.GetTransmitter()->GetParameter(ParameterName, Param))
	{
		Param.FloatParam = Default;
	}

	if(ParamMode == MPM_Direct)
	{
		return Param.FloatParam;
	}
	else if(ParamMode == MPM_Abs)
	{
		Param.FloatParam = FMath::Abs(Param.FloatParam);
	}

	float Gradient;
	if (MaxInput <= MinInput)
	{
		Gradient = 0.f;
	}
	else
	{
		Gradient = (MaxOutput - MinOutput)/(MaxInput - MinInput);
	}

	const float ClampedParam = FMath::Clamp(Param.FloatParam, MinInput, MaxInput);

	return MinOutput + ((ClampedParam - MinInput) * Gradient);
}

/*-----------------------------------------------------------------------------
	USoundNodeModulatorContinuous implementation.
-----------------------------------------------------------------------------*/
USoundNodeModulatorContinuous::USoundNodeModulatorContinuous(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

void USoundNodeModulatorContinuous::ParseNodes( FAudioDevice* AudioDevice, const UPTRINT NodeWaveInstanceHash, FActiveSound& ActiveSound, const FSoundParseParameters& ParseParams, TArray<FWaveInstance*>& WaveInstances )
{
	FSoundParseParameters UpdatedParams = ParseParams;
	UpdatedParams.Volume *= VolumeModulationParams.GetValue( ActiveSound );;
	UpdatedParams.Pitch *= PitchModulationParams.GetValue( ActiveSound );

	Super::ParseNodes( AudioDevice, NodeWaveInstanceHash, ActiveSound, UpdatedParams, WaveInstances );
}

