// Copyright Epic Games, Inc. All Rights Reserved.


#include "Sound/SoundNodeOscillator.h"
#include "ActiveSound.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SoundNodeOscillator)

/*-----------------------------------------------------------------------------
	USoundNodeOscillator implementation.
-----------------------------------------------------------------------------*/
USoundNodeOscillator::USoundNodeOscillator(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	AmplitudeMin = 0;
	AmplitudeMax = 0;
	FrequencyMin = 0;
	FrequencyMax = 0;
	OffsetMin = 0;
	OffsetMax = 0;
	CenterMin = 0;
	CenterMax = 0;
	bModulateVolume = false;
	bModulatePitch = false;
}

void USoundNodeOscillator::ParseNodes( FAudioDevice* AudioDevice, const UPTRINT NodeWaveInstanceHash, FActiveSound& ActiveSound, const FSoundParseParameters& ParseParams, TArray<FWaveInstance*>& WaveInstances )
{
	RETRIEVE_SOUNDNODE_PAYLOAD( sizeof( float ) + sizeof( float ) + sizeof( float ) + sizeof( float ) );
	DECLARE_SOUNDNODE_ELEMENT( float, UsedAmplitude );
	DECLARE_SOUNDNODE_ELEMENT( float, UsedFrequency );
	DECLARE_SOUNDNODE_ELEMENT( float, UsedOffset );
	DECLARE_SOUNDNODE_ELEMENT( float, UsedCenter );

	if( *RequiresInitialization )
	{
		UsedAmplitude = AmplitudeMax + ( ( AmplitudeMin - AmplitudeMax ) * RandomStream.FRand() );
		UsedFrequency = FrequencyMax + ( ( FrequencyMin - FrequencyMax ) * RandomStream.FRand() );
		UsedOffset = OffsetMax + ( ( OffsetMin - OffsetMax ) * RandomStream.FRand() );
		UsedCenter = CenterMax + ( ( CenterMin - CenterMax ) * RandomStream.FRand() );

		*RequiresInitialization = 0;
	}

	FSoundParseParameters UpdatedParams = ParseParams;

	const float ModulationFactor = UsedCenter + UsedAmplitude * FMath::Sin( UsedOffset + UsedFrequency * ActiveSound.PlaybackTime * UE_PI );
	if( bModulateVolume )
	{
		UpdatedParams.Volume *= ModulationFactor;
	}

	if( bModulatePitch )
	{
		UpdatedParams.Pitch *= ModulationFactor;
	}

	Super::ParseNodes( AudioDevice, NodeWaveInstanceHash, ActiveSound, UpdatedParams, WaveInstances );
}


