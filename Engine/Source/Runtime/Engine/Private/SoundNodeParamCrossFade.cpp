// Copyright Epic Games, Inc. All Rights Reserved.


#include "Sound/SoundNodeParamCrossFade.h"
#include "ActiveSound.h"
#include "IAudioParameterTransmitter.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SoundNodeParamCrossFade)

/*-----------------------------------------------------------------------------
	USoundNodeParamCrossFade implementation.
-----------------------------------------------------------------------------*/
USoundNodeParamCrossFade::USoundNodeParamCrossFade(const FObjectInitializer& ObjectInitializer) : Super(ObjectInitializer)
{
}

float USoundNodeParamCrossFade::GetCurrentDistance(FAudioDevice* AudioDevice, FActiveSound& ActiveSound, const FSoundParseParameters& ParseParams) const
{
	FAudioParameter ParamValue;
	ActiveSound.GetTransmitter()->GetParameter(ParamName, ParamValue);
	return ParamValue.FloatParam;
}

bool USoundNodeParamCrossFade::AllowCrossfading(FActiveSound& ActiveSound) const
{
	// Always allow parameter to control crossfading, even on 2D/preview sounds
	return true;
}

float USoundNodeParamCrossFade::GetMaxDistance() const
{
	float MaxDistance = 0.0f;
	for (USoundNode* ChildNode : ChildNodes)
	{
		if (ChildNode)
		{
			ChildNode->ConditionalPostLoad();
			MaxDistance = FMath::Max(ChildNode->GetMaxDistance(), MaxDistance);
		}
	}
	return MaxDistance;
}

