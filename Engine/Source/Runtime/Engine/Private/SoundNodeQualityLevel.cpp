// Copyright Epic Games, Inc. All Rights Reserved.


#include "Sound/SoundNodeQualityLevel.h"
#include "ActiveSound.h"
#include "EdGraph/EdGraph.h"
#include "Sound/AudioSettings.h"
#include "Sound/SoundCue.h"
#include "Sound/SoundNodeWavePlayer.h"
#include "AudioCompressionSettingsUtils.h"

#if WITH_EDITORONLY_DATA
#include "Settings/LevelEditorPlaySettings.h"
#include "Editor.h"
#endif
#include "Interfaces/ITargetPlatform.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SoundNodeQualityLevel)

#if WITH_EDITOR

void USoundNodeQualityLevel::ReconcileNode(bool bReconstructNode)
{
	while (ChildNodes.Num() > GetMinChildNodes())
	{
		RemoveChildNode(ChildNodes.Num()-1);
	}
	while (ChildNodes.Num() < GetMinChildNodes())
	{
		InsertChildNode(ChildNodes.Num());
	}
#if WITH_EDITORONLY_DATA
	if (GIsEditor && bReconstructNode && GraphNode)
	{
		GraphNode->ReconstructNode();
		GraphNode->GetGraph()->NotifyGraphChanged();
	}
#endif
}

FText USoundNodeQualityLevel::GetInputPinName(int32 PinIndex) const
{
	return GetDefault<UAudioSettings>()->GetQualityLevelSettings(PinIndex).DisplayName;
}
#endif

void USoundNodeQualityLevel::PostLoad()
{
	Super::PostLoad();
#if WITH_EDITOR
	ReconcileNode(false);
#endif //WITH_EDITOR

	UE_CLOGF(Cast<USoundCue>(GetOuter()) && Cast<USoundCue>(GetOuter())->GetCookedQualityIndex() != CookedQualityLevelIndex && CookedQualityLevelIndex != INDEX_NONE,
		LogAudio,
		Warning, "'%ls' has been cooked with multiple quality levels. '%ls'(%d) vs '%ls'(%d)",
		*GetFullNameSafe(this),
		*GetDefault<UAudioSettings>()->FindQualityNameByIndex(USoundCue::GetCachedQualityLevel()),
		USoundCue::GetCachedQualityLevel(),
		*GetDefault<UAudioSettings>()->FindQualityNameByIndex(CookedQualityLevelIndex),
		CookedQualityLevelIndex
	);
}

void USoundNodeQualityLevel::PrimeChildWavePlayers(bool bRecurse)
{
	ForCurrentQualityLevel([bRecurse](USoundNode* Node){ Node->PrimeChildWavePlayers(bRecurse); });
}

void USoundNodeQualityLevel::RetainChildWavePlayers(bool bRecurse)
{
	ForCurrentQualityLevel([bRecurse](USoundNode* Node) { Node->RetainChildWavePlayers(bRecurse); });
}

void USoundNodeQualityLevel::ReleaseRetainerOnChildWavePlayers(bool bRecurse)
{
	ForCurrentQualityLevel([bRecurse](USoundNode* Node) { Node->ReleaseRetainerOnChildWavePlayers(bRecurse); });
}

void USoundNodeQualityLevel::LoadChildWavePlayers(bool bAddToRoot, bool bRecurse)
{
	ForCurrentQualityLevel([bAddToRoot, bRecurse](USoundNode* Node) 
	{ 
		if (Node)
		{
			// If this node is a wave player node, load it, otherwise load children 
			// (a wave player node cannot have children)
			if (USoundNodeWavePlayer* WavePlayerNode = Cast<USoundNodeWavePlayer>(Node))
			{
				WavePlayerNode->LoadAsset(bAddToRoot);
			}
			else
			{
				Node->LoadChildWavePlayerAssets(bAddToRoot, bRecurse);
			}
		}
	});
}

int32 USoundNodeQualityLevel::GetMaxChildNodes() const
{
	return GetDefault<UAudioSettings>()->QualityLevels.Num();
}

int32 USoundNodeQualityLevel::GetMinChildNodes() const
{
	return GetDefault<UAudioSettings>()->QualityLevels.Num();
}

void USoundNodeQualityLevel::ParseNodes( FAudioDevice* AudioDevice, const UPTRINT NodeWaveInstanceHash, FActiveSound& ActiveSound, const FSoundParseParameters& ParseParams, TArray<FWaveInstance*>& WaveInstances )
{
#if WITH_EDITOR
	int32 QualityLevel = 0;

	if (GIsEditor)
	{
		RETRIEVE_SOUNDNODE_PAYLOAD( sizeof( int32 ) );
		DECLARE_SOUNDNODE_ELEMENT( int32, CachedQualityLevel );

		if (*RequiresInitialization)
		{
			const bool bIsPIESound = ((GEditor->bIsSimulatingInEditor || GEditor->PlayWorld != nullptr) && ActiveSound.GetWorldID() > 0);
			if (bIsPIESound)
			{
				CachedQualityLevel = GetDefault<ULevelEditorPlaySettings>()->PlayInEditorSoundQualityLevel;
			}
		}

		QualityLevel = CachedQualityLevel;
	}
	else
	{
		QualityLevel = USoundCue::GetCachedQualityLevel();
	}
#else
	
	int32 QualityLevel = USoundCue::GetCachedQualityLevel();
	
	// If CookedQualityLevelIndex has been set, we will have a *single* quality level.
	if (CookedQualityLevelIndex >= 0 && ChildNodes.Num() == 1)
	{	
		// Remap to index 0 (as all other levels have been removed by cooker).
		QualityLevel = 0;
	}
#endif	
	
	if (ChildNodes.IsValidIndex(QualityLevel) && ChildNodes[QualityLevel])
	{
		ChildNodes[QualityLevel]->ParseNodes( AudioDevice,
			GetNodeWaveInstanceHash(NodeWaveInstanceHash, ChildNodes[QualityLevel], QualityLevel), ActiveSound, ParseParams, WaveInstances );
	}
}

void USoundNodeQualityLevel::Serialize(FArchive& Ar)
{
#if WITH_EDITOR

	if (Ar.IsCooking() && Ar.IsSaving() && Ar.CookingTarget() )
	{			
		if (const FPlatformAudioCookOverrides* CookOverrides = FPlatformCompressionUtilities::GetCookOverrides(*Ar.CookingTarget()->IniPlatformName()))
		{
			// Prevent any other thread saving this class while we are modifying the ChildNode array.
			FScopeLock Lock(&EditorOnlyCs);

			if (CookOverrides->SoundCueCookQualityIndex != INDEX_NONE )
			{
				// Set our cook quality, as we serialize. 
				CookedQualityLevelIndex = CookOverrides->SoundCueCookQualityIndex;

				// Move out all nodes.
				TArray<TObjectPtr<class USoundNode>> ChildNodesBackup;
				ChildNodesBackup = MoveTemp(ChildNodes);
				check(ChildNodes.Num() == 0);
			
				// Put *just* the node we care about in our child array to be serialized by the Super
				if (ChildNodesBackup.IsValidIndex(CookedQualityLevelIndex))
				{
					ChildNodes.Add(ChildNodesBackup[CookedQualityLevelIndex]);
				}

				int32 BranchesPruned = ChildNodesBackup.Num() - ChildNodes.Num();
				UE_CLOGF(
					BranchesPruned > 0,
					LogAudio,
					Display,
					"Pruning '%ls' of '%d' quality branches, as it's cooked at '%ls' quality.",
					*GetFullNameSafe(this),
					BranchesPruned,
					*GetDefault<UAudioSettings>()->FindQualityNameByIndex(CookedQualityLevelIndex)
				);

				// Call base serialize that will walk all properties and serialize them.
				Super::Serialize(Ar);

				// Return to our original state. (careful the cook only variables don't leak out).
				ChildNodes = MoveTemp(ChildNodesBackup);
				CookedQualityLevelIndex = INDEX_NONE;

				// We are done.
				return;
			}
		}	
	}
		
#endif //WITH_EDITOR	

	// ... in all other cases, we just call the super.
	Super::Serialize(Ar);
}

void USoundNodeQualityLevel::ForCurrentQualityLevel(TFunction<void(USoundNode*)>&& Lambda)
{
	// If we're able to retrieve a valid cached quality level for this sound cue,
	// only release that quality level.
	int32 QualityLevel = USoundCue::GetCachedQualityLevel();

#if WITH_EDITOR
	if (GIsEditor && QualityLevel < 0)
	{
		QualityLevel = GetDefault<ULevelEditorPlaySettings>()->PlayInEditorSoundQualityLevel;
	}
#endif

	// If CookedQualityLevelIndex has been set, we will have a *single* quality level.
	if (CookedQualityLevelIndex >= 0 && ChildNodes.Num() == 1)
	{
		// Remap to index 0 (as all other levels have been removed by cooker).
		QualityLevel = 0;
	}
	
	if (ChildNodes.IsValidIndex(QualityLevel) && ChildNodes[QualityLevel])
	{
		Lambda(ChildNodes[QualityLevel]);
	}
}

