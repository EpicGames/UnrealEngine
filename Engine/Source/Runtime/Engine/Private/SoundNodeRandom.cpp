// Copyright Epic Games, Inc. All Rights Reserved.


#include "Sound/SoundNodeRandom.h"
#include "ActiveSound.h"
#include "AudioCompressionSettingsUtils.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SoundNodeRandom)

#if WITH_EDITOR
	#include "Editor.h"
#endif

static int32 MaxRandomBranchesCVar = 0;
FAutoConsoleVariableRef CVarMaxRandomBranches(
	TEXT("au.MaxRandomBranches"),
	MaxRandomBranchesCVar,
	TEXT("Sets the max amount of branches to play from for any random node. The rest of the branches will be released from memory.\n")
	TEXT("0: No culling, Any other value: The amount of branches we should use as a maximum for any random node."),
	ECVF_Default);

static int32 PrimeRandomSoundNodesCVar = 0;
FAutoConsoleVariableRef CVarPrimeRandomSoundNodes(
	TEXT("au.streamcache.priming.PrimeRandomNodes"),
	PrimeRandomSoundNodesCVar,
	TEXT("When set to 1, sounds will be loaded into the cache automatically when a random node is hit.\n"),
	ECVF_Default);

/*-----------------------------------------------------------------------------
    USoundNodeRandom implementation.
-----------------------------------------------------------------------------*/
USoundNodeRandom::USoundNodeRandom(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bRandomizeWithoutReplacement = true;
	NumRandomUsed = 0;
}

void USoundNodeRandom::FixWeightsArray()
{
	// If weights and children got out of sync, we fix it first.
	if( Weights.Num() < ChildNodes.Num() )
	{
		Weights.AddZeroed( ChildNodes.Num() - Weights.Num() );
	}
	else if( Weights.Num() > ChildNodes.Num() )
	{
		const int32 NumToRemove = Weights.Num() - ChildNodes.Num();
		Weights.RemoveAt( Weights.Num() - NumToRemove, NumToRemove );
	}
}

void USoundNodeRandom::FixHasBeenUsedArray()
{
	// If HasBeenUsed and children got out of sync, we fix it first.
	if( HasBeenUsed.Num() < ChildNodes.Num() )
	{
		HasBeenUsed.AddZeroed( ChildNodes.Num() - HasBeenUsed.Num() );
	}
	else if( HasBeenUsed.Num() > ChildNodes.Num() )
	{
		const int32 NumToRemove = HasBeenUsed.Num() - ChildNodes.Num();
		HasBeenUsed.RemoveAt( HasBeenUsed.Num() - NumToRemove, NumToRemove );
	}
}

void USoundNodeRandom::PostLoad()
{
	Super::PostLoad();

	// get a per-platform override:
	if (!bShouldExcludeFromBranchCulling && !bSoundCueExcludedFromBranchCulling)
	{
		int32 AmountOfBranchesToPreselect = DetermineAmountOfBranchesToPreselect();

		// Use the amount we determined above to remove branches:
		if (!GIsEditor && AmountOfBranchesToPreselect > 0)
		{
			// Cull branches from the end:
			int32 LastIndex = ChildNodes.Num() - 1;
			while (ChildNodes.Num() > AmountOfBranchesToPreselect)
			{
				RemoveChildNode(LastIndex--);
			}
		}
#if WITH_EDITOR
		else if (GEditor != nullptr && (GEditor->bIsSimulatingInEditor || GEditor->PlayWorld != NULL))
		{
			UpdatePIEHiddenNodes();
		}
#endif //WITH_EDITOR
	}

	FixWeightsArray();
	FixHasBeenUsedArray();
}

void USoundNodeRandom::PostEditImport()
{
	Super::PostEditImport();
	
	FixWeightsArray();
	FixHasBeenUsedArray();
}

int32 USoundNodeRandom::ChooseNodeIndex(FActiveSound& ActiveSound)
{
	int32 NodeIndex = 0;
	float WeightSum = 0.0f;

#if WITH_EDITOR
	bool bIsPIESound = (GEditor != nullptr) && ((GEditor->bIsSimulatingInEditor || GEditor->PlayWorld != nullptr) && ActiveSound.GetWorldID() > 0);

	if (bIsPIESound)
	{
		// Find the first available index - needed if there is only one
		while (PIEHiddenNodes.Contains(NodeIndex))
		{
			NodeIndex++;
		}
	}
#endif //WITH_EDITOR


	// only calculate the weights that have not been used and use that set for the random choice
	for (int32 i = 0; i < Weights.Num(); ++i)
	{
#if WITH_EDITOR
		if (!bIsPIESound || !PIEHiddenNodes.Contains(i))
#endif //WITH_EDITOR
		{
			if (!bRandomizeWithoutReplacement || !HasBeenUsed[i])
			{
				WeightSum += Weights[i];
			}
		}
	}

	float Choice = FMath::FRand() * WeightSum;
	WeightSum = 0.0f;
	for (int32 i = 0; i < ChildNodes.Num() && i < Weights.Num(); ++i)
	{
#if WITH_EDITOR
		if (!bIsPIESound || !PIEHiddenNodes.Contains(i))
#endif //WITH_EDITOR
		{
			if (bRandomizeWithoutReplacement && HasBeenUsed[i])
			{
				continue;
			}
			WeightSum += Weights[i];
			if (Choice < WeightSum)
			{
				NodeIndex = i;
				HasBeenUsed[i] = true;
				NumRandomUsed++;
				break;
			}
		}
	}

	return NodeIndex;
}

int32 USoundNodeRandom::DetermineAmountOfBranchesToPreselect()
{
	int32 AmountOfBranchesToPreselect = 0;
	int32 OverrideForAmountOfBranchesToPreselect = 0;


	if (MaxRandomBranchesCVar > 0)
	{
		// If the CVar has been set, allow it to override our .ini setting:
		OverrideForAmountOfBranchesToPreselect = MaxRandomBranchesCVar;
	}
	else
	{
		// Otherwise, we use the value from the ini setting, if we have one:
		OverrideForAmountOfBranchesToPreselect = FPlatformCompressionUtilities::GetMaxPreloadedBranchesForCurrentPlatform();
	}

	if (PreselectAtLevelLoad > 0 && OverrideForAmountOfBranchesToPreselect > 0)
	{
		// If we have to decide between the override and this node's PreselectAtLevelLoad property,
		// use the minimum of either:
		AmountOfBranchesToPreselect = FMath::Min(PreselectAtLevelLoad, OverrideForAmountOfBranchesToPreselect);
	}
	else if (PreselectAtLevelLoad > 0)
	{
		AmountOfBranchesToPreselect = PreselectAtLevelLoad;
	}
	else
	{
		// Otherwise, just use the override:
		AmountOfBranchesToPreselect = OverrideForAmountOfBranchesToPreselect;
	}

	return AmountOfBranchesToPreselect;
}

void USoundNodeRandom::ParseNodes( FAudioDevice* AudioDevice, const UPTRINT NodeWaveInstanceHash, FActiveSound& ActiveSound, const FSoundParseParameters& ParseParams, TArray<FWaveInstance*>& WaveInstances )
{
	RETRIEVE_SOUNDNODE_PAYLOAD(sizeof(int32));
	DECLARE_SOUNDNODE_ELEMENT(int32, NodeIndex);

	// Pick a random child node and save the index.
	if (*RequiresInitialization)
	{
		if (PrimeRandomSoundNodesCVar != 0)
		{
			PrimeChildWavePlayers(true);
		}

		NodeIndex = ChooseNodeIndex(ActiveSound);
		*RequiresInitialization = 0;
	}

#if WITH_EDITOR
	bool bIsPIESound = (GEditor != nullptr) && ((GEditor->bIsSimulatingInEditor || GEditor->PlayWorld != nullptr) && ActiveSound.GetWorldID() > 0);
#endif //WITH_EDITOR

	// check to see if we have used up our random sounds
	if (bRandomizeWithoutReplacement && (HasBeenUsed.Num() > 0) && (NumRandomUsed >= HasBeenUsed.Num()
#if WITH_EDITOR
		|| (bIsPIESound && NumRandomUsed >= (HasBeenUsed.Num() - PIEHiddenNodes.Num()))
#endif //WITH_EDITOR
		))
	{
		// reset all of the children nodes
		for (int32 i = 0; i < HasBeenUsed.Num(); ++i)
		{
			if (HasBeenUsed.Num() > NodeIndex)
			{
				HasBeenUsed[i] = false;
			}
		}

		// set the node that has JUST played to be true so we don't repeat it
		if (HasBeenUsed.IsValidIndex(NodeIndex))
		{
			HasBeenUsed[NodeIndex] = true;
		}
		NumRandomUsed = 1;
	}

	if (NodeIndex < ChildNodes.Num() && ChildNodes[NodeIndex])
	{
		ChildNodes[NodeIndex]->ParseNodes(AudioDevice, GetNodeWaveInstanceHash(NodeWaveInstanceHash, ChildNodes[NodeIndex], NodeIndex), ActiveSound, ParseParams, WaveInstances);
	}
}

int32 USoundNodeRandom::GetNumSounds(const UPTRINT NodeWaveInstanceHash, FActiveSound& ActiveSound) const
{
	RETRIEVE_SOUNDNODE_PAYLOAD(sizeof(int32));
	DECLARE_SOUNDNODE_ELEMENT(int32, NodeIndex);

	// Pick a random child node and save the index.
	if (*RequiresInitialization)
	{
		// Unfortunately, ChooseNodeIndex modifies USoundNodeRandom data
		NodeIndex = const_cast<USoundNodeRandom*>(this)->ChooseNodeIndex(ActiveSound);
		*RequiresInitialization = 0;
	}

	check(!*RequiresInitialization);

	if (NodeIndex < ChildNodes.Num() && ChildNodes[NodeIndex])
	{
		const UPTRINT ChildNodeWaveInstanceHash = GetNodeWaveInstanceHash(NodeWaveInstanceHash, ChildNodes[NodeIndex], NodeIndex);
		return ChildNodes[NodeIndex]->GetNumSounds(ChildNodeWaveInstanceHash, ActiveSound);
	}
	return 0;
}

void USoundNodeRandom::CreateStartingConnectors()
{
	// Random Sound Nodes default with two connectors.
	InsertChildNode( ChildNodes.Num() );
	InsertChildNode( ChildNodes.Num() );
}

void USoundNodeRandom::InsertChildNode( int32 Index )
{
	FixWeightsArray();
	FixHasBeenUsedArray();

	check( Index >= 0 && Index <= Weights.Num() );
	check( ChildNodes.Num() == Weights.Num() );

	Weights.InsertUninitialized( Index );
	Weights[Index] = 1.0f;

	HasBeenUsed.InsertUninitialized( Index );
	HasBeenUsed[ Index ] = false;

	Super::InsertChildNode( Index );
}

void USoundNodeRandom::RemoveChildNode( int32 Index )
{
	FixWeightsArray();
	FixHasBeenUsedArray();

	check( Index >= 0 && Index < Weights.Num() );
	check( ChildNodes.Num() == Weights.Num() );

	Weights.RemoveAt( Index );
	HasBeenUsed.RemoveAt( Index );

	Super::RemoveChildNode( Index );
}

#if WITH_EDITOR
void USoundNodeRandom::SetChildNodes(TArray<USoundNode*>& InChildNodes)
{
	Super::SetChildNodes(InChildNodes);

	if (Weights.Num() < ChildNodes.Num())
	{
		while (Weights.Num() < ChildNodes.Num())
		{
			int32 NewIndex = Weights.Num();
			Weights.InsertUninitialized(NewIndex);
			Weights[ NewIndex ] = 1.0f;
		}
	}
	else if (Weights.Num() > ChildNodes.Num())
	{
		const int32 NumToRemove = Weights.Num() - ChildNodes.Num();
		Weights.RemoveAt(Weights.Num() - NumToRemove, NumToRemove);
	}

	if (HasBeenUsed.Num() < ChildNodes.Num())
	{
		while (HasBeenUsed.Num() < ChildNodes.Num())
		{
			int32 NewIndex = HasBeenUsed.Num();
			HasBeenUsed.InsertUninitialized(NewIndex);
			HasBeenUsed[ NewIndex ] = false;
		}
	}
	else if (HasBeenUsed.Num() > ChildNodes.Num())
	{
		const int32 NumToRemove = HasBeenUsed.Num() - ChildNodes.Num();
		HasBeenUsed.RemoveAt(HasBeenUsed.Num() - NumToRemove, NumToRemove);
	}

}

void USoundNodeRandom::OnBeginPIE(const bool bIsSimulating)
{
	UpdatePIEHiddenNodes();
}
#endif //WITH_EDITOR

#if WITH_EDITOR
void USoundNodeRandom::UpdatePIEHiddenNodes()
{
	// should we hide some nodes?
	int32 NodesToHide = ChildNodes.Num() - PreselectAtLevelLoad;
	if (PreselectAtLevelLoad > 0 && NodesToHide > 0)
	{
		// Choose the right amount of nodes to hide
		PIEHiddenNodes.Empty();
		while (PIEHiddenNodes.Num() < NodesToHide)
		{
			PIEHiddenNodes.AddUnique(FMath::Rand() % ChildNodes.Num());
		}
		// reset all of the child nodes and the use count
		for( int32 i = 0; i < HasBeenUsed.Num(); ++i )
		{
			HasBeenUsed[ i ] = false;
		}
		NumRandomUsed = 0;
	}
	// don't hide zero/negative amounts of nodes
	else if ((PreselectAtLevelLoad <= 0 || NodesToHide <= 0))
	{
		PIEHiddenNodes.Empty();
	}
}
#endif //WITH_EDITOR

