// Copyright Epic Games, Inc. All Rights Reserved.


#include "Sound/SoundNodeSoundClass.h"
#include "ActiveSound.h"
#include "Sound/SoundClass.h"
#include "Sound/SoundWaveLoadingBehavior.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SoundNodeSoundClass)

/*-----------------------------------------------------------------------------
	USoundNodeSoundClass implementation.
-----------------------------------------------------------------------------*/

USoundNodeSoundClass::USoundNodeSoundClass(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, bRetainingAudioDueToSoundClass(false)
{
}

void USoundNodeSoundClass::ParseNodes( class FAudioDevice* AudioDevice, const UPTRINT NodeWaveInstanceHash, FActiveSound& ActiveSound, const FSoundParseParameters& ParseParams, TArray<FWaveInstance*>& WaveInstances )
{
	FSoundParseParameters UpdatedParseParams = ParseParams;
	if (SoundClassOverride)
	{
		UpdatedParseParams.SoundClass = SoundClassOverride;
	}

	Super::ParseNodes( AudioDevice, NodeWaveInstanceHash, ActiveSound, UpdatedParseParams, WaveInstances );
}

void USoundNodeSoundClass::PostLoad()
{
	Super::PostLoad();
	
	ESoundWaveLoadingBehavior SoundClassLoadingBehavior = ESoundWaveLoadingBehavior::Inherited;

	USoundClass* CurrentSoundClass = SoundClassOverride;

	// Recurse through this sound class's parents until we find an override.
	while (SoundClassLoadingBehavior == ESoundWaveLoadingBehavior::Inherited && CurrentSoundClass != nullptr)
	{
		SoundClassLoadingBehavior = CurrentSoundClass->Properties.LoadingBehavior;
		CurrentSoundClass = CurrentSoundClass->ParentClass;
	}

	if (!GIsEditor && SoundClassLoadingBehavior == ESoundWaveLoadingBehavior::RetainOnLoad)
	{
		RetainChildWavePlayers(true);
		bRetainingAudioDueToSoundClass = true;
	}
	else if (SoundClassLoadingBehavior == ESoundWaveLoadingBehavior::PrimeOnLoad)
	{
		PrimeChildWavePlayers(true);
	}
}

void USoundNodeSoundClass::BeginDestroy()
{
	Super::BeginDestroy();
}

