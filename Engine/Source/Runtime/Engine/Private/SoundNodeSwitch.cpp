// Copyright Epic Games, Inc. All Rights Reserved.


#include "Sound/SoundNodeSwitch.h"
#include "ActiveSound.h"
#include "IAudioParameterTransmitter.h"
#include "Sound/SoundCue.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SoundNodeSwitch)

#define LOCTEXT_NAMESPACE "SoundNodeSwitch"

USoundNodeSwitch::USoundNodeSwitch(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

void USoundNodeSwitch::ParseNodes(FAudioDevice* AudioDevice, const UPTRINT NodeWaveInstanceHash, FActiveSound& ActiveSound, const FSoundParseParameters& ParseParams, TArray<FWaveInstance*>& WaveInstances)
{
	FAudioParameter Param;
	if (ActiveSound.GetTransmitter()->GetParameter(IntParameterName, Param))
	{
		Param.IntParam += 1;
	}
	
	if (Param.IntParam < 0 || Param.IntParam >= ChildNodes.Num())
	{
		Param.IntParam = 0;
	}

	if (Param.IntParam < ChildNodes.Num() && ChildNodes[Param.IntParam])
	{
		ChildNodes[Param.IntParam]->ParseNodes(AudioDevice, GetNodeWaveInstanceHash(NodeWaveInstanceHash, ChildNodes[Param.IntParam], Param.IntParam), ActiveSound, ParseParams, WaveInstances);
	}
}

void USoundNodeSwitch::CreateStartingConnectors()
{
	InsertChildNode(ChildNodes.Num());
	InsertChildNode(ChildNodes.Num());
	InsertChildNode(ChildNodes.Num());
	InsertChildNode(ChildNodes.Num());
}

#if WITH_EDITOR
void USoundNodeSwitch::RenamePins()
{
#if WITH_EDITORONLY_DATA
	USoundCue::GetSoundCueAudioEditor()->RenameNodePins(this);
#endif
}

FText USoundNodeSwitch::GetInputPinName(int32 PinIndex) const
{
	if (PinIndex == 0)
	{
		return LOCTEXT("ParamUnset", "Parameter Unset");
	}
	else
	{
		return FText::FromString(FString::Printf(TEXT("%d"), PinIndex - 1));
	}
}

FText USoundNodeSwitch::GetTitle() const
{
	FFormatNamedArguments Arguments;
	Arguments.Add(TEXT("Description"), Super::GetTitle());
	Arguments.Add(TEXT("ParameterName"), FText::FromName(IntParameterName));

	return FText::Format(LOCTEXT("Title", "{Description} ({ParameterName})"), Arguments);
}
#endif //WITH_EDITOR

#undef LOCTEXT_NAMESPACE

