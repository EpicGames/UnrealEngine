// Copyright Epic Games, Inc. All Rights Reserved.


#include "Sound/SoundNodeWaveParam.h"
#include "ActiveSound.h"
#include "AudioDevice.h"
#include "IAudioParameterTransmitter.h"
#include "Sound/SoundCue.h"
#include "Sound/SoundWave.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SoundNodeWaveParam)

/*-----------------------------------------------------------------------------
	USoundNodeWaveParam implementation
-----------------------------------------------------------------------------*/
USoundNodeWaveParam::USoundNodeWaveParam(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

float USoundNodeWaveParam::GetDuration()
{
	// Since we can't know how long this node will be we say it is indefinitely looping
	return INDEFINITELY_LOOPING_DURATION;
}

void USoundNodeWaveParam::ParseNodes(FAudioDevice* AudioDevice, const UPTRINT NodeWaveInstanceHash, FActiveSound& ActiveSound, const FSoundParseParameters& ParseParams, TArray<FWaveInstance*>& WaveInstances)
{
	FAudioParameter ParamValue;
	ActiveSound.GetTransmitter()->GetParameter(WaveParameterName, ParamValue);

	if (USoundWave* NewWave = Cast<USoundWave>(ParamValue.ObjectParam))
	{
		RETRIEVE_SOUNDNODE_PAYLOAD(sizeof(USoundWave*));
		DECLARE_SOUNDNODE_ELEMENT(USoundWave*, PrevWave);

		const UPTRINT WaveHash = GetNodeWaveInstanceHash(NodeWaveInstanceHash, (UPTRINT)NewWave, 0);
		
		if (PrevWave != NewWave)
		{
			if (FSoundCueParameterTransmitter* SoundCueTransmitter = static_cast<FSoundCueParameterTransmitter*>(ActiveSound.GetTransmitter()))
			{
				// removing here prevents waste in the case that a metasound is replaced with a soundwave
				SoundCueTransmitter->RemoveChildTransmitter(GetNodeWaveInstanceHash(NodeWaveInstanceHash, (UPTRINT)PrevWave, 0));
				
				Audio::FParameterTransmitterInitParams Params;
				Params.InstanceID = Audio::GetTransmitterID(ActiveSound.GetAudioComponentID(), WaveHash, ActiveSound.GetPlayOrder()); 
				Params.SampleRate = AudioDevice->GetSampleRate();
				Params.AudioDeviceID = AudioDevice->DeviceID;

				SoundCueTransmitter->CreateChildTransmitter(WaveHash, *NewWave, MoveTemp(Params));
			}

			PrevWave = NewWave;
		}
		
		NewWave->Parse(AudioDevice, WaveHash, ActiveSound, ParseParams, WaveInstances);
	}
	else
	{
		// use the default node linked to us, if any
		Super::ParseNodes(AudioDevice, NodeWaveInstanceHash, ActiveSound, ParseParams, WaveInstances);
	}
}


