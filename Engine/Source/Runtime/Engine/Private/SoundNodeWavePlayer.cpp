// Copyright Epic Games, Inc. All Rights Reserved.

#include "Sound/SoundNodeWavePlayer.h"
#include "ActiveSound.h"
#include "AudioDevice.h"
#include "Sound/SoundCue.h"
#include "Sound/SoundWave.h"
#include "UObject/FrameworkObjectVersion.h"
#include "Async/Async.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SoundNodeWavePlayer)

#define LOCTEXT_NAMESPACE "SoundNodeWavePlayer"

void USoundNodeWavePlayer::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

	Ar.UsingCustomVersion(FFrameworkObjectVersion::GUID);

	if (Ar.CustomVer(FFrameworkObjectVersion::GUID) >= FFrameworkObjectVersion::HardSoundReferences)
	{
		if (Ar.IsLoading())
		{
			Ar << SoundWave;
		}
		else if (Ar.IsSaving())
		{
			USoundWave* HardReference = (ShouldHardReferenceAsset(Ar.CookingTarget()) ? ToRawPtr(SoundWave) : nullptr);
			Ar << HardReference;
		}
	}
}

bool USoundNodeWavePlayer::ContainsProceduralSoundReference() const
{
	if (SoundWave)
	{
		return SoundWave->IsA<USoundWaveProcedural>();
	}
	return false;
}

void USoundNodeWavePlayer::LoadAsset(bool bAddToRoot)
{
	if (IsAsyncLoadingMultithreaded())
	{
		SoundWave = SoundWaveAssetPtr.Get();
		if (SoundWave && SoundWave->HasAnyFlags(RF_NeedLoad))
		{
			// This can happen when the owning USoundCue's PostLoad gets called and the SoundWave hasn't been serialized yet
			// In this case we need to make sure we don't pass the pointer to the SoundNodeWavePlayer too early as the SoundWave
			// will be serialized on the AsyncLoadingThread shortly and this may lead to strange race conditions / thread safety issues
			SoundWave = nullptr;
		}
		if (SoundWave == nullptr)
		{
			const FString LongPackageName = SoundWaveAssetPtr.GetLongPackageName();
			if (!LongPackageName.IsEmpty())
			{
				UE_LOGF(LogAudio, VeryVerbose, " '%ls:%ls', Async loading...", *GetNameSafe(GetOuter()), *GetName());
				bAsyncLoading = true;
				LoadPackageAsync(LongPackageName, FLoadPackageAsyncDelegate::CreateUObject(this, &USoundNodeWavePlayer::OnSoundWaveLoaded, bAddToRoot));
			}
		}
		else if (bAddToRoot)
		{
			SoundWave->AddToRoot();
		}
		if (SoundWave)
		{
			SoundWave->AddToCluster(this, true);
			// Don't init resources when running cook, as this can trigger 
			// registration of a MetaSound and its dependent graphs.
			// Those will instead be registered when the MetaSound itself is cooked (FMetasoundAssetBase::CookMetaSound)
			// in a way that does not deal with runtime data like this function does
			if (!IsRunningCookCommandlet())
			{
				SoundWave->InitResources();
			}
		}
	}
	else
	{
		UE_LOGF(LogAudio, VeryVerbose, "'%ls:%ls', DOING SYNCRONOUS loading... %ls",
			*GetNameSafe(GetOuter()),
			*GetName(),
			SoundWaveAssetPtr.IsPending() ? TEXT("pending.") : TEXT("not-pending.")
		);

		SoundWave = SoundWaveAssetPtr.LoadSynchronous();
		if (SoundWave)
		{
			if (bAddToRoot)
			{
				SoundWave->AddToRoot();
			}
			SoundWave->AddToCluster(this);
			// Don't init resources when running cook, as this can trigger 
			// registration of a MetaSound and its dependent graphs.
			// Those will instead be registered when the MetaSound itself is cooked (FMetasoundAssetBase::CookMetaSound)
			// in a way that does not deal with runtime data like this function does
			if (!IsRunningCookCommandlet())
			{
				SoundWave->InitResources();
			}
		}
	}
}

void USoundNodeWavePlayer::ClearAssetReferences()
{
	SoundWave = nullptr;
}

void USoundNodeWavePlayer::OnSoundWaveLoaded(const FName& PackageName, UPackage* Package, EAsyncLoadingResult::Type Result, bool bAddToRoot)
{
	if (Result == EAsyncLoadingResult::Succeeded)
	{
		SoundWave = SoundWaveAssetPtr.Get();
		if (SoundWave)
		{
			if (bAddToRoot)
			{
				SoundWave->AddToRoot();
			}
			SoundWave->AddToCluster(this);
		}
	}
	bAsyncLoading = false;
}

void USoundNodeWavePlayer::SetSoundWave(USoundWave* InSoundWave)
{
	SoundWave = InSoundWave;
	SoundWaveAssetPtr = InSoundWave;
}

#if WITH_EDITOR
void USoundNodeWavePlayer::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	// Undo calls this and reports null PropertyChangedEvent,
	// (PostEditUndo is not guaranteed to be called post re-serialization
	// of parent SoundCue, but PostEditChangeProperty is)
	// so always ensure asset is loaded on any call.
	// Secondary calls will be fast and result in no-op.
	LoadAsset();
}
#endif // WITH_EDITOR

void USoundNodeWavePlayer::ParseNodes( FAudioDevice* AudioDevice, const UPTRINT NodeWaveInstanceHash, FActiveSound& ActiveSound, const FSoundParseParameters& ParseParams, TArray<FWaveInstance*>& WaveInstances )
{
	if (bAsyncLoading)
	{
		UE_LOGF(LogAudio, Verbose, "Asynchronous load of %ls not complete in USoundNodeWavePlayer::ParseNodes, will attempt to play later.", *GetFullNameSafe(this));
		// We're still loading so don't stop this active sound yet
		ActiveSound.bFinished = false;
		return;
	}

	if (SoundWave)
	{
		RETRIEVE_SOUNDNODE_PAYLOAD(sizeof(int32));
		DECLARE_SOUNDNODE_ELEMENT(int32, bPlayFailed);

		if (*RequiresInitialization)
		{
			bPlayFailed = 0;

			if (FSoundCueParameterTransmitter* SoundCueTransmitter = static_cast<FSoundCueParameterTransmitter*>(ActiveSound.GetTransmitter()))
			{
				Audio::FParameterTransmitterInitParams Params;
				Params.InstanceID = Audio::GetTransmitterID(ActiveSound.GetAudioComponentID(), NodeWaveInstanceHash, ActiveSound.GetPlayOrder()); 
				Params.SampleRate = AudioDevice->GetSampleRate();
				Params.AudioDeviceID = AudioDevice->DeviceID;

				SoundCueTransmitter->CreateChildTransmitter(NodeWaveInstanceHash, *SoundWave, MoveTemp(Params));
			}
			
			*RequiresInitialization = 0;
		}

		// The SoundWave's bLooping is only for if it is directly referenced, so clear it
		// in the case that it is being played from a player
		const bool bWaveIsLooping = SoundWave->bLooping;
		SoundWave->bLooping = false;

		if (bLooping || (SoundWave->bProcedural && !SoundWave->IsOneShot()))
		{
			FSoundParseParameters UpdatedParams = ParseParams;
			UpdatedParams.bLooping = true;
			SoundWave->Parse(AudioDevice, NodeWaveInstanceHash, ActiveSound, UpdatedParams, WaveInstances);
		}
		else if (ParseParams.bEnableRetrigger)
		{
			// Don't play non-looping sounds again if this sound has been revived to// avoid re-triggering one shots played adjacent with looping wave instances in cue
			SoundWave->Parse(AudioDevice, NodeWaveInstanceHash, ActiveSound, ParseParams, WaveInstances);
		}
		else
		{
			// If sound has been virtualized, don't try to revive one-shots
			if (!ActiveSound.bHasVirtualized)
			{
				// Guard against continual parsing if wave instance was created but not added to transient
				// wave instance list to avoid inaudible sounds popping back in.
				if (bPlayFailed == 0)
				{
					const int32 InitActiveSoundWaveInstanceNum = ActiveSound.GetWaveInstances().Num();
					const int32 InitWaveInstancesNum = WaveInstances.Num();
					
					SoundWave->Parse(AudioDevice, NodeWaveInstanceHash, ActiveSound, ParseParams, WaveInstances);
					
					const bool bFailed = ActiveSound.GetWaveInstances().Num() == InitActiveSoundWaveInstanceNum && WaveInstances.Num() == InitWaveInstancesNum;
					if (bFailed)
					{
						bPlayFailed = 1;
					}
				}
			}
		}

		SoundWave->bLooping = bWaveIsLooping;
	}
	else if (!SoundWaveAssetPtr.IsNull() && !bAsyncLoadRequestPending)
	{
		UE_LOGF(LogAudio, Warning, "Asynchronous load of %ls required in USoundNodeWavePlayer::ParseNodes when we attempted to play, likely because the quality node was changed.", *GetFullNameSafe(this));
		
		// raise this flag in case this node is parsed again before the game thread task queue is executed.
		bAsyncLoadRequestPending = true;

		TWeakObjectPtr<USoundNodeWavePlayer> WeakThis = MakeWeakObjectPtr(this);

		// Dispatch a call to the game thread to load this asset.
		AsyncTask(ENamedThreads::GameThread, [WeakThis]() {
			if (WeakThis.IsValid())
			{
				WeakThis->LoadAsset();
				WeakThis->bAsyncLoadRequestPending = false;
			}
		});	
	}
}

float USoundNodeWavePlayer::GetDuration()
{
	float Duration = 0.f;
	if (SoundWave)
	{
		if (bLooping)
		{
			Duration = INDEFINITELY_LOOPING_DURATION;
		}
		else
		{
			Duration = SoundWave->Duration;
		}
	}
	return Duration;
}

bool USoundNodeWavePlayer::IsPlayWhenSilent() const
{
	if (SoundWave)
	{
		return SoundWave->VirtualizationMode == EVirtualizationMode::PlayWhenSilent;
	}
	return false;
}

#if WITH_EDITOR
FText USoundNodeWavePlayer::GetTitle() const
{
	FText SoundWaveName;
	if (SoundWave)
	{
		SoundWaveName = FText::FromString(SoundWave->GetFName().ToString());
	}
	else if (SoundWaveAssetPtr.IsValid())
	{
		SoundWaveName = FText::FromString(SoundWaveAssetPtr.GetAssetName());
	}
	else
	{
		SoundWaveName = LOCTEXT("NoSoundWave", "NONE");
	}

	FText Title;

	FFormatNamedArguments Arguments;
	Arguments.Add(TEXT("Description"), Super::GetTitle());
	Arguments.Add(TEXT("SoundWaveName"), SoundWaveName);
	if (bLooping)
	{
		Title = FText::Format(LOCTEXT("LoopingSoundWaveDescription", "Looping {Description} : {SoundWaveName}"), Arguments);
	}
	else
	{
		Title = FText::Format(LOCTEXT("NonLoopingSoundWaveDescription", "{Description} : {SoundWaveName}"), Arguments);
	}

	return Title;
}
#endif

// A Wave Player is the end of the chain and has no children
int32 USoundNodeWavePlayer::GetMaxChildNodes() const
{
	return 0;
}


#undef LOCTEXT_NAMESPACE
