// Copyright Epic Games, Inc. All Rights Reserved.


#include "Sound/SoundSourceBus.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SoundSourceBus)

void FSoundSourceBusSendInfo::PostSerialize(const FArchive& Ar)
{
#if WITH_EDITORONLY_DATA
	if (Ar.IsLoading())
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS

		if (LPFCutoffFrequency_DEPRECATED.IsSet() && !bEnableLPFCutoff)
		{
			bEnableLPFCutoff = 1;
			LPFCutoff = LPFCutoffFrequency_DEPRECATED.GetValue();
		}
		LPFCutoffFrequency_DEPRECATED.Reset();

		if (HPFCutoffFrequency_DEPRECATED.IsSet() && !bEnableHPFCutoff)
		{
			bEnableHPFCutoff = 1;
			HPFCutoff = HPFCutoffFrequency_DEPRECATED.GetValue();
		}
		HPFCutoffFrequency_DEPRECATED.Reset();

		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}
#endif // WITH_EDITORONLY_DATA
}

USoundSourceBus::USoundSourceBus(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	// This is a bus. This will result in the decompression type to be set as DTYPE_Bus. Audio won't be generated from this object but from instance data in audio mixer.
	bIsSourceBus = true;

	Init();
}

void USoundSourceBus::PostLoad()
{
	Super::PostLoad();

	Init();
}

void USoundSourceBus::Init()
{
	// Allow users to manually set the source bus duration
	Duration = GetDuration();

	// This sound wave is looping if the source bus duration is 0.0f
	bLooping = (SourceBusDuration == 0.0f);

	// Set the channels equal to the users channel count choice
	switch (SourceBusChannels)
	{
	case ESourceBusChannels::Mono:
		NumChannels = 1;
		break;

	case ESourceBusChannels::Stereo:
		NumChannels = 2;
		break;
	}
}

#if WITH_EDITOR
void USoundSourceBus::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Init();
	MaxDistance = ComputeMaxDistance();
}
#endif

void USoundSourceBus::Serialize(FArchive& InArchive)
{
	Super::Serialize(InArchive);
	
#if WITH_EDITORONLY_DATA
	if (InArchive.IsLoading())
	{
		// Source Buses do not support pitch modulation, so remove any connections if users previously attempted it.
		ModulationSettings.PitchRouting = EModulationRouting::Disable;
		ModulationSettings.PitchModulationDestination.Modulators.Empty();
	}
#endif // WITH_EDITORONLY_DATA
}

bool USoundSourceBus::IsPlayable() const
{
	return true;
}

float USoundSourceBus::GetDuration() const
{
	return (SourceBusDuration > 0.0f) ? SourceBusDuration : INDEFINITELY_LOOPING_DURATION;
}
