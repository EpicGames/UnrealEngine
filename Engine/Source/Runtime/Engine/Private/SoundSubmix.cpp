// Copyright Epic Games, Inc. All Rights Reserved.
#include "Sound/SoundSubmix.h"

#include "AudioDevice.h"
#include "Engine/Engine.h"
#include "Engine/World.h"
#include "Sound/SampleBufferIO.h"
#include "Stats/Stats.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SoundSubmix)

#if WITH_EDITOR
#include "Framework/Notifications/NotificationManager.h"
#include "IAudioEndpoint.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "ISoundfieldEndpoint.h"
#include "Subsystems/AssetEditorSubsystem.h"
#include "Editor.h"
#include "Async/Async.h"
#endif // WITH_EDITOR

#include "SoundSubmixCustomVersion.h"

static int32 ClearBrokenSubmixAssetsCVar = 0;
FAutoConsoleVariableRef CVarFixUpBrokenSubmixAssets(
	TEXT("au.submix.clearbrokensubmixassets"),
	ClearBrokenSubmixAssetsCVar,
	TEXT("If set, will verify that we don't have a submix that lists a child submix that is no longer its child, and the former children will not erroneously list their previous parents.\n")
	TEXT("0: Disable, >0: Enable"),
	ECVF_Default);

namespace SoundSubmixPrivate
{
	// Modulators default. 
	static const float Default_OutputVolumeModulation = 0.f;
	static const float Default_WetLevelModulation = 0.f;
	static const float Default_DryLevelModulation = -96.f;
}

USoundSubmixWithParentBase::USoundSubmixWithParentBase(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, ParentSubmix(nullptr)
	, bIsDynamic(0)
{}

USoundSubmixBase::USoundSubmixBase(const FObjectInitializer& ObjectInitializer)
#if WITH_EDITORONLY_DATA
	: SoundSubmixGraph(nullptr)
#endif // WITH_EDITORONLY_DATA
{}

USoundSubmix::USoundSubmix(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, bMuteWhenBackgrounded(0)
	, AmbisonicsPluginSettings(nullptr)
	, EnvelopeFollowerAttackTime(10)
	, EnvelopeFollowerReleaseTime(500)
{
	using namespace SoundSubmixPrivate;
	OutputVolumeModulation.Value	= Default_OutputVolumeModulation;
	WetLevelModulation.Value		= Default_WetLevelModulation;
	DryLevelModulation.Value		= Default_DryLevelModulation;

#if WITH_EDITORONLY_DATA
	InitDeprecatedDefaults();
#endif //WITH_EDITORONLY_DATA
}

void USoundSubmix::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);
	Ar.UsingCustomVersion(FSoundSubmixCustomVersion::GUID);
}

void USoundSubmix::PostLoad()
{
	Super::PostLoad();

#if WITH_EDITORONLY_DATA

	const int32 Version = GetLinkerCustomVersion(FSoundSubmixCustomVersion::GUID);
	HandleVersionMigration(Version);

#endif // WITH_EDITORONLY_DATA

}

UEndpointSubmix::UEndpointSubmix(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, EndpointType(IAudioEndpointFactory::GetTypeNameForDefaultEndpoint())
{

}

USoundfieldSubmix::USoundfieldSubmix(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, SoundfieldEncodingFormat(ISoundfieldFactory::GetFormatNameForInheritedEncoding())
{}

USoundfieldEndpointSubmix::USoundfieldEndpointSubmix(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, SoundfieldEndpointType(ISoundfieldEndpointFactory::DefaultSoundfieldEndpointName())
{}

void USoundSubmix::StartRecordingOutput(const UObject* WorldContextObject, float ExpectedDuration)
{
	if (FAudioDevice* AudioDevice = GetAudioDeviceFrom(WorldContextObject))
	{
		StartRecordingOutput(AudioDevice, ExpectedDuration);
	}
}

void USoundSubmix::StartRecordingOutput(FAudioDevice* InDevice, float ExpectedDuration)
{
	if (InDevice)
	{
		InDevice->StartRecording(this, ExpectedDuration);
	}
}

void USoundSubmix::StopRecordingOutput(const UObject* WorldContextObject, EAudioRecordingExportType ExportType, const FString& Name, FString Path, USoundWave* ExistingSoundWaveToOverwrite)
{
	if (FAudioDevice* AudioDevice = GetAudioDeviceFrom(WorldContextObject))
	{
		StopRecordingOutput(AudioDevice, ExportType, Name, Path, ExistingSoundWaveToOverwrite);
	}
}

void USoundSubmix::StopRecordingOutput(FAudioDevice* InDevice, EAudioRecordingExportType ExportType, const FString& Name, FString Path, USoundWave* ExistingSoundWaveToOverwrite /*= nullptr*/)
{
	if (InDevice)
	{
		float SampleRate;
		float ChannelCount;

		Audio::AlignedFloatBuffer& RecordedBuffer = InDevice->StopRecording(this, ChannelCount, SampleRate);

		// This occurs when Stop Recording Output is called when Start Recording Output was not called.
		if (RecordedBuffer.Num() == 0)
		{
			return;
		}

		// Pack output data into DSPSampleBuffer and record it out!
		RecordingData.Reset(new Audio::FAudioRecordingData());

		RecordingData->InputBuffer = Audio::TSampleBuffer<int16>(RecordedBuffer, ChannelCount, SampleRate);

		switch (ExportType)
		{
			case EAudioRecordingExportType::SoundWave:
			{
				// If we're using the editor, we can write out a USoundWave to the content directory. Otherwise, we just generate a USoundWave without writing it to disk.
				if (GIsEditor)
				{
					RecordingData->Writer.BeginWriteToSoundWave(Name, RecordingData->InputBuffer, Path, [this](const USoundWave* Result)
					{
						if (OnSubmixRecordedFileDone.IsBound())
						{
							OnSubmixRecordedFileDone.Broadcast(Result);
						}
					});
				}
				else
				{
					RecordingData->Writer.BeginGeneratingSoundWaveFromBuffer(RecordingData->InputBuffer, nullptr, [this](const USoundWave* Result)
					{
						if (OnSubmixRecordedFileDone.IsBound())
						{
							OnSubmixRecordedFileDone.Broadcast(Result);
						}
					});
				}
			}
			break;
			
			case EAudioRecordingExportType::WavFile:
			{
				RecordingData->Writer.BeginWriteToWavFile(RecordingData->InputBuffer, Name, Path, [this]()
				{
					if (OnSubmixRecordedFileDone.IsBound())
					{
						OnSubmixRecordedFileDone.Broadcast(nullptr);
					}
				});
			}
			break;

			default:
			break;
		}
	}
}

void USoundSubmix::StartEnvelopeFollowing(const UObject* WorldContextObject)
{
	if (FAudioDevice* AudioDevice = GetAudioDeviceFrom(WorldContextObject))
	{
		StartEnvelopeFollowing(AudioDevice);
	}
}

void USoundSubmix::StartEnvelopeFollowing(FAudioDevice* InAudioDevice)
{
	if (InAudioDevice)
	{
		InAudioDevice->StartEnvelopeFollowing(this);
	}
}

void USoundSubmix::StopEnvelopeFollowing(const UObject* WorldContextObject)
{
	if (FAudioDevice* AudioDevice = GetAudioDeviceFrom(WorldContextObject))
	{
		StopEnvelopeFollowing(AudioDevice);
	}
}

void USoundSubmix::StopEnvelopeFollowing(FAudioDevice* InAudioDevice)
{
	if (InAudioDevice)
	{
		InAudioDevice->StopEnvelopeFollowing(this);
	}
}

void USoundSubmix::AddEnvelopeFollowerDelegate(const UObject* WorldContextObject, const FOnSubmixEnvelopeBP& OnSubmixEnvelopeBP)
{
	if (FAudioDevice* AudioDevice = GetAudioDeviceFrom(WorldContextObject))
	{
		AudioDevice->AddEnvelopeFollowerDelegate(this, OnSubmixEnvelopeBP);
	}
}

void USoundSubmix::RemoveEnvelopeFollowerDelegate(const UObject* WorldContextObject, const FOnSubmixEnvelopeBP& OnSubmixEnvelopeBP)
{
	if (FAudioDevice* AudioDevice = GetAudioDeviceFrom(WorldContextObject))
	{
		AudioDevice->RemoveEnvelopeFollowerDelegate(this, OnSubmixEnvelopeBP);
	}
}

void USoundSubmix::AddSpectralAnalysisDelegate(const UObject* WorldContextObject, const TArray<FSoundSubmixSpectralAnalysisBandSettings>& InBandSettings, const FOnSubmixSpectralAnalysisBP& OnSubmixSpectralAnalysisBP, float UpdateRate,  float DecibelNoiseFloor, bool bDoNormalize, bool bDoAutoRange, float AutoRangeAttackTime, float AutoRangeReleaseTime)
{
	if (FAudioDevice* AudioDevice = GetAudioDeviceFrom(WorldContextObject))
	{
		FSoundSpectrumAnalyzerDelegateSettings DelegateSettings = USoundSubmix::GetSpectrumAnalysisDelegateSettings(InBandSettings, UpdateRate, DecibelNoiseFloor, bDoNormalize, bDoAutoRange, AutoRangeAttackTime, AutoRangeReleaseTime);
		AudioDevice->AddSpectralAnalysisDelegate(this, DelegateSettings, OnSubmixSpectralAnalysisBP);
	}
}

void USoundSubmix::RemoveSpectralAnalysisDelegate(const UObject* WorldContextObject, const FOnSubmixSpectralAnalysisBP& OnSubmixSpectralAnalysisBP)
{
	if (FAudioDevice* AudioDevice = GetAudioDeviceFrom(WorldContextObject))
	{
		AudioDevice->RemoveSpectralAnalysisDelegate(this, OnSubmixSpectralAnalysisBP);
	}
}

void USoundSubmix::StartSpectralAnalysis(const UObject* WorldContextObject, EFFTSize FFTSize, EFFTPeakInterpolationMethod InterpolationMethod, EFFTWindowType WindowType, float HopSize, EAudioSpectrumType SpectrumType)
{
	if (FAudioDevice* AudioDevice = GetAudioDeviceFrom(WorldContextObject))
	{
		StartSpectralAnalysis(AudioDevice, FFTSize, InterpolationMethod, WindowType, HopSize, SpectrumType);
	}
}

void USoundSubmix::StartSpectralAnalysis(FAudioDevice* InAudioDevice, EFFTSize FFTSize, EFFTPeakInterpolationMethod InterpolationMethod, EFFTWindowType WindowType, float HopSize, EAudioSpectrumType SpectrumType)
{
	if (InAudioDevice)
	{
		FSoundSpectrumAnalyzerSettings Settings = USoundSubmix::GetSpectrumAnalyzerSettings(FFTSize, InterpolationMethod, WindowType, HopSize, SpectrumType);
		InAudioDevice->StartSpectrumAnalysis(this, Settings);
	}
}

void USoundSubmix::StopSpectralAnalysis(const UObject* WorldContextObject)
{
	if (FAudioDevice* AudioDevice = GetAudioDeviceFrom(WorldContextObject))
	{
		AudioDevice->StopSpectrumAnalysis(this);
	}
}

void USoundSubmix::StopSpectralAnalysis(FAudioDevice* InAudioDevice)
{
	if (InAudioDevice)
	{
		InAudioDevice->StopSpectrumAnalysis(this);
	}
}

void USoundSubmix::SetSubmixOutputVolume(const UObject* WorldContextObject, float InOutputVolume)
{
	if (FAudioDevice* AudioDevice = GetAudioDeviceFrom(WorldContextObject))
	{
		AudioDevice->SetSubmixOutputVolume(this, InOutputVolume);
	}
}

void USoundSubmix::SetSubmixWetLevel(const UObject* WorldContextObject, float InWetLevel)
{
	if (FAudioDevice* AudioDevice = GetAudioDeviceFrom(WorldContextObject))
	{
		AudioDevice->SetSubmixWetLevel(this, InWetLevel);
	}
}

void USoundSubmix::SetSubmixDryLevel(const UObject* WorldContextObject, float InDryLevel)
{
	if (FAudioDevice* AudioDevice = GetAudioDeviceFrom(WorldContextObject))
	{
		AudioDevice->SetSubmixDryLevel(this, InDryLevel);
	}
}

#if WITH_EDITOR
void USoundSubmix::PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent)
{
	if (PropertyChangedEvent.Property != nullptr)
	{
		// Force the properties to be initialized for this SoundSubmix on all active audio devices
		if (FAudioDeviceManager* AudioDeviceManager = FAudioDeviceManager::Get())
		{
			FName MemberName = PropertyChangedEvent.MemberProperty->GetFName();

			if (MemberName == GET_MEMBER_NAME_CHECKED(USoundSubmix, bAutoDisable))
			{
				const bool NewAutoDisable = bAutoDisable;
				USoundSubmix* SoundSubmix = this;
				AudioDeviceManager->IterateOverAllDevices([SoundSubmix, NewAutoDisable](Audio::FDeviceId Id, FAudioDevice* Device)
				{
					Device->SetSubmixAutoDisable(SoundSubmix, NewAutoDisable);
				});
			}

			if (MemberName == GET_MEMBER_NAME_CHECKED(USoundSubmix, AutoDisableTime))
			{
				const float NewAutoDisableTime = AutoDisableTime;
				USoundSubmix* SoundSubmix = this;
				AudioDeviceManager->IterateOverAllDevices([SoundSubmix, NewAutoDisableTime](Audio::FDeviceId Id, FAudioDevice* Device)
				{
					Device->SetSubmixAutoDisableTime(SoundSubmix, NewAutoDisableTime);
				});
			}

			if (MemberName == GET_MEMBER_NAME_CHECKED(USoundSubmix, OutputVolumeModulation)
				|| MemberName == GET_MEMBER_NAME_CHECKED(USoundSubmix, WetLevelModulation)
				|| MemberName == GET_MEMBER_NAME_CHECKED(USoundSubmix, DryLevelModulation))
			{
				USoundSubmix* SoundSubmix = this;

				PushModulationChanges();

				float NewVolumeModBase = OutputVolumeModulation.Value;
				float NewWetModBase = WetLevelModulation.Value;
				float NewDryModBase = DryLevelModulation.Value;

				AudioDeviceManager->IterateOverAllDevices([SoundSubmix, NewVolumeModBase, NewWetModBase, NewDryModBase](Audio::FDeviceId Id, FAudioDevice* Device) 
				{
					Device->SetSubmixModulationBaseLevels(SoundSubmix, NewVolumeModBase, NewWetModBase, NewDryModBase);
				});
			}

			if (MemberName == GET_MEMBER_NAME_CHECKED(USoundSubmix, SubmixEffectChain))
			{
				AudioDeviceManager->RegisterSoundSubmix(this);
			}
		}
	}

	Super::PostEditChangeProperty(PropertyChangedEvent);
}
#endif

void USoundSubmix::SetOutputVolumeModulation(const FSoundModulationDestinationSettings& InVolMod)
{
	OutputVolumeModulation = InVolMod;
	PushModulationChanges();
}

void USoundSubmix::SetWetVolumeModulation(const FSoundModulationDestinationSettings& InVolMod)
{
	WetLevelModulation = InVolMod;
	PushModulationChanges();
}

void USoundSubmix::SetDryVolumeModulation(const FSoundModulationDestinationSettings& InVolMod)
{
	DryLevelModulation = InVolMod;
	PushModulationChanges();
}

void USoundSubmix::PushModulationChanges()
{
	if (FAudioDeviceManager* AudioDeviceManager = FAudioDeviceManager::Get())
	{
		// Send the changes to the Modulation System
		TSet<TObjectPtr<USoundModulatorBase>> NewVolumeMod = OutputVolumeModulation.Modulators;
		TSet<TObjectPtr<USoundModulatorBase>> NewWetLevelMod = WetLevelModulation.Modulators;
		TSet<TObjectPtr<USoundModulatorBase>> NewDryLevelMod = DryLevelModulation.Modulators;
		AudioDeviceManager->IterateOverAllDevices([SoundSubmix = this, VolMod = MoveTemp(NewVolumeMod), WetMod = MoveTemp(NewWetLevelMod), DryMod = MoveTemp(NewDryLevelMod)](Audio::FDeviceId Id, FAudioDevice* Device) mutable
		{
		    if (Device)
		    {
		    	Device->UpdateSubmixModulationSettings(SoundSubmix, VolMod, WetMod, DryMod);
			}
		});
	}
}

FAudioDevice* USoundSubmix::GetAudioDeviceFrom(const UObject* WorldContextObject) 
{
	if (GEngine)
	{
		if (UWorld* ThisWorld = GEngine->GetWorldFromContextObject(WorldContextObject, EGetWorldErrorMode::LogAndReturnNull))
		{
			return ThisWorld->GetAudioDeviceRaw();
		}
	}

	return nullptr;
}

void USoundSubmixBase::IncrementLoadRefCount()
{
	const int32 OldRefCount = RefCount.fetch_add(1, std::memory_order_relaxed);
	if (OldRefCount == 0)
	{
		AddToRoot();
	}
}

void USoundSubmixBase::DecrementLoadRefCount()
{
	const int32 OldRefCount = RefCount.fetch_sub(1, std::memory_order_relaxed);
	if (OldRefCount == 1 && IsRooted())
	{
		RemoveFromRoot();
	}
}

FString USoundSubmixBase::GetDesc()
{
	return FString("Sound Submix");
}

void USoundSubmixBase::BeginDestroy()
{
	// Use the main/default audio device for storing and retrieving sound class properties
	FAudioDeviceManager* AudioDeviceManager = (GEngine ? GEngine->GetAudioDeviceManager() : nullptr);

	// Force the properties to be initialized for this SoundClass on all active audio devices
	if (AudioDeviceManager)
	{
		AudioDeviceManager->UnregisterSoundSubmix(this);
	}

	// This has to be called AFTER device unregistration.
	// Otherwise, the object can be in a partially destroyed state.
	Super::BeginDestroy();
}

void USoundSubmixBase::PostLoad()
{
	Super::PostLoad();

	if (ClearBrokenSubmixAssetsCVar)
	{
		for (int32 ChildIndex = ChildSubmixes.Num() - 1; ChildIndex >= 0; ChildIndex--)
		{
			USoundSubmixBase* ChildSubmix = ChildSubmixes[ChildIndex];

			if (!ChildSubmix)
			{
				continue;
			}

			if (USoundSubmixWithParentBase* CastedChildSubmix = Cast<USoundSubmixWithParentBase>(ChildSubmix))
			{
				if (!ensure(CastedChildSubmix->ParentSubmix == this))
				{
					UE_LOGF(LogAudio, Warning, "Submix had a child submix that didn't explicitly mark this submix as a parent!");
					ChildSubmixes.RemoveAtSwap(ChildIndex);
				}
			}
			else
			{
				ensureMsgf(false, TEXT("Submix had a child submix that doesn't have an output!"));
				ChildSubmixes.RemoveAtSwap(ChildIndex);
			}
		}
	}

	// Use the main/default audio device for storing and retrieving sound class properties
	FAudioDeviceManager* AudioDeviceManager = (GEngine ? GEngine->GetAudioDeviceManager() : nullptr);

	// Force the properties to be initialized for this SoundClass on all active audio devices
	const FString PathName = GetPathName();

	// Do not support auto-registration for submixes in the temp directory
	// (to avoid issues with validation & automatically rooting objects on load).
	const bool bIsTemp = PathName.StartsWith(TEXT("/Temp/"));

	// Dynamic submixes register via DynamicConnect, skip them on asset load.
	const bool bIsDynamicSubmix = IsDynamic(true /* bIncludeAncestors */);

	if (AudioDeviceManager && !bIsTemp && !bIsDynamicSubmix)
	{
		AudioDeviceManager->RegisterSoundSubmix(this);
	}
}

TObjectPtr<USoundSubmixBase> USoundSubmixWithParentBase::GetParent(Audio::FDeviceId InDeviceId) const
{
	// Dynamic parent?
	if (const TObjectPtr<USoundSubmixBase>* pFound = DynamicParentSubmix.Find(InDeviceId))
	{
		return *pFound;
	}
	return ParentSubmix;
}

bool USoundSubmixWithParentBase::DynamicConnect(const UObject* WorldContextObject, USoundSubmixBase* InParent)
{	
	if (!WorldContextObject)
	{
		UE_LOGF(LogAudio, Warning, "Submix (DynamicConnect): World Context is null for [%ls]", *GetName());
		return false;
	}
		
	const UWorld* World = WorldContextObject->GetWorld();
	if (!World)
	{
		UE_LOGF(LogAudio, Warning, "Submix (DynamicConnect): World is null for [%ls]", *GetName());		
		return false;
	}

	return DynamicConnect(World->GetAudioDevice(), InParent);
}

bool USoundSubmixWithParentBase::DynamicConnect(FAudioDeviceHandle Handle, USoundSubmixBase* InParent)
{
	if (!IsDynamic(false /* bIncludeAncestors */))
	{
		const USoundSubmixBase* DynamicAncestor = FindDynamicAncestor();
		UE_CLOGF(DynamicAncestor, LogAudio, Warning, "Submix (DynamicConnect): Dynamic Flag not set for [%ls], you need its ancestor [%ls]. Call FindDynamicAncestor on this submix to find it. Ignoring...", *GetName(), *DynamicAncestor->GetName());
		UE_CLOGF(!DynamicAncestor, LogAudio, Warning, "Submix (DynamicConnect): Dynamic Flag not set for [%ls] or any of its parents, ignoring... ", *GetName());
		return false;
	}
	if (!Handle.IsValid())
	{
		UE_LOGF(LogAudio, Warning, "Submix (DynamicConnect): No valid audio device in this world for [%ls]", *GetName());
		return false;
	}

	// Reject anything that's Soundfield related. That system is being retired/
	if (Cast<USoundfieldSubmix>(this) || Cast<USoundfieldSubmix>(InParent))
	{
		UE_LOGF(LogAudio, Warning, "Submix (DynamicConnect): Soundfield submixes do not support dynamic connections. Submix=%ls, Parent=%ls ", *GetName(), *GetNameSafe(InParent));
		return false;
	}
	
	if (InParent && !SubmixUtils::AreSubmixFormatsCompatible(this, InParent))
	{
		UE_LOGF(LogAudio, Warning, "Submix (DynamicConnect): Submix connot be connected are they are incompatible [%ls] and [%ls]", *GetName(), *GetNameSafe(InParent));
		return false;
	}

	// Already part of the graph?
	if (InParent && SubmixUtils::FindInGraph(InParent, this, true, Handle))
	{
		UE_LOGF(LogAudio, Warning, "Submix (DynamicConnect): Submix [%ls] is already part of the graph", *GetName());
		return false;
	}

	const Audio::DeviceID Id = Handle.GetDeviceID();

	TObjectPtr<USoundSubmixBase>& CurrentParent = DynamicParentSubmix.FindOrAdd(Id);

	if (CurrentParent != InParent)
	{

		if (CurrentParent)
		{
			CurrentParent->DynamicChildSubmixes.FindOrAdd(Id).ChildSubmixes.Remove(this);
			UE_LOGF(LogAudio, Verbose, "Submix (DynamicConnect): Reparenting [%ls] and removing from it's parent [%ls]", *GetName(), *CurrentParent->GetName());
		}

		CurrentParent = InParent;
		if (CurrentParent)
		{
			CurrentParent->DynamicChildSubmixes.FindOrAdd(Id).ChildSubmixes.AddUnique(this);
			UE_LOGF(LogAudio, Verbose, "Submix (DynamicConnect): Reparenting [%ls] and adding to it's new parent [%ls]", *GetName(), *CurrentParent->GetName());

			// Disable our parents auto disable feature.
			Handle->SetSubmixAutoDisable(Cast<USoundSubmix>(CurrentParent.Get()), false);
		}

		// Register us and our children
		SubmixUtils::ForEachStaticChildRecursive(
			this,
			[&Handle, Id](USoundSubmixBase* Iter)-> void
			{
				UE_LOGF(LogAudio, Verbose, "Submix (DynamicConnect): Registering [%ls] with AudioDevice [%u]", *Iter->GetName(), Id);
				Handle->RegisterSoundSubmix(Iter, /* bInit*/ true);
			});
			
		// ... and disable parents auto disable feature.
		Handle->SetSubmixAutoDisable(Cast<USoundSubmix>(this), CurrentParent == nullptr);



		return CurrentParent != nullptr;
	}

	UE_CLOGF(InParent, LogAudio, Warning, "Submix (DynamicConnect): Submix [%ls] was already connected to [%ls]", *GetName(), *GetNameSafe(CurrentParent));
	UE_CLOGF(InParent == nullptr, LogAudio, Warning, "Submix (DynamicConnect): Connecting a [%ls] to a null parent, but our parent is already null", *GetName());
	return false;

}

bool USoundSubmixWithParentBase::DynamicDisconnect(const UObject* WorldContextObject)
{
	if (!WorldContextObject)
	{
		UE_LOGF(LogAudio, Warning, "Submix (DynamicDisconnect): World Context is null for [%ls]", *GetName());
		return false;
	}
		
	const UWorld* World = WorldContextObject->GetWorld();
	if (!World)
	{
		UE_LOGF(LogAudio, Warning, "Submix (DynamicDisconnect): World is null for [%ls]", *GetName());		
		return false;
	}

	return DynamicDisconnect(World->GetAudioDevice());
}

bool USoundSubmixWithParentBase::DynamicDisconnect(FAudioDeviceHandle Handle)
{
	if (!Handle.IsValid())
	{
		UE_LOGF(LogAudio, Warning, "Submix (DynamicDisconnect): No valid audio device in this world for [%ls]", *GetName());
		return false;
	}

	if (!IsDynamic(false /* bIncludeAncestors */))
	{
		UE_LOGF(LogAudio, Warning, "Submix (DynamicDisconnect): Dynamic Flag not set for [%ls] ignoring.", *GetName());
		return false;
	}

	const Audio::DeviceID Id = Handle.GetDeviceID();

	TObjectPtr<USoundSubmixBase>& CurrentParent = DynamicParentSubmix.FindOrAdd(Id);

	if (CurrentParent)
	{
		UE_LOGF(LogAudio, Verbose, "Submix (DynamicDisconnect): Removing [%ls] from it's parent [%ls]", *GetName(), *CurrentParent->GetName());

		CurrentParent->DynamicChildSubmixes.FindOrAdd(Id).ChildSubmixes.Remove(this);
		CurrentParent = nullptr;

		 SubmixUtils::ForEachStaticChildRecursive(
			this,
			[&Handle](USoundSubmixBase* Iter)-> void
			{
				Handle->UnregisterSoundSubmix(Iter,  /* bReparentChildren*/ false);
			});	
				
		// If we still have a valid parent static submix? Make sure that's still live and registered.
		if (ParentSubmix)
		{
			Handle->RegisterSoundSubmix(this, false);
		}
		
		UE_LOGF(LogAudio, Verbose, "Submix (DynamicDisconnect): Unregistering [%ls] with AudioDevice [%u]", *GetName(), Id);

		return true;
	}

	UE_LOGF(LogAudio, Warning, "Submix (DynamicDisconnect): Submix was not connected to any dynamic parent [%ls]", *GetName());
	return false;
}

bool USoundSubmixWithParentBase::IsDynamic(const bool bIncludeAncestors) const
{
	// If we don't care about ancestors, just return if we're dynamic.
	if (!bIncludeAncestors)
	{
		return bIsDynamic;
	}

	// Find the first dynamic ancestor.
	const USoundSubmixBase* Found = FindDynamicAncestor();
	return Found != nullptr;
}

USoundSubmixBase* USoundSubmixWithParentBase::FindDynamicAncestor()
{
	const USoundSubmixBase* Found = const_cast<const USoundSubmixWithParentBase*>(this)->FindDynamicAncestor();
	return const_cast<USoundSubmixBase*>(Found);
}

const USoundSubmixBase* USoundSubmixWithParentBase::FindDynamicAncestor() const
{
	// Walk up parents from here checking for dynamic flag.
	for (const USoundSubmixWithParentBase* Current = this; Current; /* Incremented below */)
	{
		if (Current->bIsDynamic)
		{
			return Current;
		}
		if (const USoundSubmixWithParentBase* Parent = Cast<USoundSubmixWithParentBase>(Current->ParentSubmix))
		{
			Current = Parent;
		}
		else
		{
			break;
		}
	}
	return nullptr;
}

#if WITH_EDITOR

void USoundSubmixBase::PostDuplicate(EDuplicateMode::Type DuplicateMode)
{
	if (DuplicateMode == EDuplicateMode::Normal)
	{
		ChildSubmixes.Reset();
	}
}

void USoundSubmixBase::PreEditChange(FProperty* PropertyAboutToChange)
{
	if (PropertyAboutToChange && PropertyAboutToChange->GetFName() == GET_MEMBER_NAME_CHECKED(USoundSubmixBase, ChildSubmixes))
	{
		// Take a copy of the current state of child classes
		BackupChildSubmixes = ChildSubmixes;
	}
}

void USoundSubmixBase::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	if (!GEngine)
	{
		return;
	}

	if (PropertyChangedEvent.Property != nullptr)
	{
		if (PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(USoundSubmixBase, ChildSubmixes))
		{
			// Find child that was changed/added
			for (int32 ChildIndex = 0; ChildIndex < ChildSubmixes.Num(); ChildIndex++)
			{
				if (ChildSubmixes[ChildIndex] != nullptr && !BackupChildSubmixes.Contains(ChildSubmixes[ChildIndex]))
				{
					if (SubmixUtils::FindInGraph(this, ChildSubmixes[ChildIndex], false))
					{
						// Contains cycle so revert to old layout - launch notification to inform user
						FNotificationInfo Info(NSLOCTEXT("Engine", "UnableToChangeSoundSubmixChildDueToInfiniteLoopNotification", "Could not change SoundSubmix child as it would create a loop"));
						Info.ExpireDuration = 5.0f;
						Info.Image = FCoreStyle::Get().GetBrush(TEXT("MessageLog.Error"));
						FSlateNotificationManager::Get().AddNotification(Info);

						// Revert to the child submixes
						ChildSubmixes = BackupChildSubmixes;
					}
					else if (USoundSubmixWithParentBase* SubmixWithParent = CastChecked<USoundSubmixWithParentBase>(ChildSubmixes[ChildIndex]))
					{
						// Update parentage
						SubmixWithParent->SetParentSubmix(this);
					}
					
					ChildSubmixes[ChildIndex]->PostEditChangeProperty(PropertyChangedEvent);

					break;
				}
			}

			// Update old child's parent if it has been removed
			for (int32 ChildIndex = 0; ChildIndex < BackupChildSubmixes.Num(); ChildIndex++)
			{
				if (BackupChildSubmixes[ChildIndex] != nullptr && !ChildSubmixes.Contains(BackupChildSubmixes[ChildIndex]))
				{
					BackupChildSubmixes[ChildIndex]->Modify();
					if (USoundSubmixWithParentBase* SubmixWithParent = Cast<USoundSubmixWithParentBase>(BackupChildSubmixes[ChildIndex]))
					{
						SubmixWithParent->ParentSubmix = nullptr;
					}

					// Force the properties to be initialized for this SoundSubmix on all active audio devices
					if (FAudioDeviceManager* AudioDeviceManager = GEngine->GetAudioDeviceManager())
					{
						if (!IsDynamic(true /* bIncludeAncestors */ )) // Exclude dynamic submixes from registration
						{
							AudioDeviceManager->RegisterSoundSubmix(this);
						}
					}
				}
			}
		}
	}

	BackupChildSubmixes.Reset();

	Super::PostEditChangeProperty(PropertyChangedEvent);
}

TArray<TObjectPtr<USoundSubmixBase>> USoundSubmixBase::BackupChildSubmixes;
#endif


void USoundSubmixWithParentBase::SetParentSubmix(USoundSubmixBase* InParentSubmix, bool bModifyAssets)
{
	if (ParentSubmix != InParentSubmix)
	{
		if (ParentSubmix)
		{
			if (bModifyAssets)
			{
				ParentSubmix->Modify();
			}
			ParentSubmix->ChildSubmixes.Remove(this);
		}

		if (bModifyAssets)
		{
			Modify();
		}
		ParentSubmix = InParentSubmix;
		if (ParentSubmix)
		{
			ParentSubmix->ChildSubmixes.AddUnique(this);
		}
	}
}


#if WITH_EDITOR
void USoundSubmixWithParentBase::PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent)
{
	if (!GEngine)
	{
		return;
	}

	if (PropertyChangedEvent.Property != nullptr)
	{
		FName ChangedPropName = PropertyChangedEvent.Property->GetFName();

		if (ChangedPropName == GET_MEMBER_NAME_CHECKED(USoundSubmixWithParentBase, ParentSubmix))
		{
			// Add this sound class to the parent class if it's not already added
			if (ParentSubmix)
			{
				bool bIsChildSubmix = false;
				for (int32 i = 0; i < ParentSubmix->ChildSubmixes.Num(); ++i)
				{
					USoundSubmixBase* ChildSubmix = ParentSubmix->ChildSubmixes[i];
					if (ChildSubmix && ChildSubmix == this)
					{
						bIsChildSubmix = true;
						break;
					}
				}

				if (!bIsChildSubmix)
				{
					ParentSubmix->Modify();
					ParentSubmix->ChildSubmixes.AddUnique(this);
				}
			}

			Modify();

			// Force the properties to be initialized for this SoundSubmix on all active audio devices
			if (FAudioDeviceManager* AudioDeviceManager = GEngine->GetAudioDeviceManager())
			{
				AudioDeviceManager->RegisterSoundSubmix(this);
			}
		}
	}

	Super::PostEditChangeProperty(PropertyChangedEvent);
}

void USoundSubmixWithParentBase::PostDuplicate(EDuplicateMode::Type DuplicateMode)
{
	if (DuplicateMode == EDuplicateMode::Normal)
	{
		SetParentSubmix(nullptr);
	}

	Super::PostDuplicate(DuplicateMode);
}

void USoundSubmixBase::AddReferencedObjects(UObject* InThis, FReferenceCollector& Collector)
{
	USoundSubmixBase* This = CastChecked<USoundSubmixBase>(InThis);

	Collector.AddReferencedObject(This->SoundSubmixGraph, This);

	for (auto& Backup : This->BackupChildSubmixes)
	{
		Collector.AddReferencedObject(Backup);
	}

	Super::AddReferencedObjects(InThis, Collector);
}
#endif // WITH_EDITOR

ISoundfieldFactory* USoundfieldSubmix::GetSoundfieldFactoryForSubmix() const
{
	// If this isn't called in the game thread, a ParentSubmix could get destroyed while we are recursing through the submix graph.
	// ensure(IsInGameThread());

	FName SoundfieldFormat = GetSubmixFormat();
	check(SoundfieldFormat != ISoundfieldFactory::GetFormatNameForInheritedEncoding());

	return ISoundfieldFactory::Get(SoundfieldFormat);
}

const USoundfieldEncodingSettingsBase* USoundfieldSubmix::GetSoundfieldEncodingSettings() const
{
	return GetEncodingSettings();
}

TArray<USoundfieldEffectBase *> USoundfieldSubmix::GetSoundfieldProcessors() const
{
	return SoundfieldEffectChain;
}

FName USoundfieldSubmix::GetSubmixFormat() const
{
	USoundfieldSubmix* ParentSoundfieldSubmix = Cast<USoundfieldSubmix>(ParentSubmix);

	if (!ParentSoundfieldSubmix || SoundfieldEncodingFormat != ISoundfieldFactory::GetFormatNameForInheritedEncoding())
	{
		if (SoundfieldEncodingFormat == ISoundfieldFactory::GetFormatNameForInheritedEncoding())
		{
			return ISoundfieldFactory::GetFormatNameForNoEncoding();
		}
		else
		{
			return SoundfieldEncodingFormat;
		}

	}
	else if(ParentSoundfieldSubmix)
	{
		// If this submix matches the format of whatever submix it's plugged into, 
		// Recurse into the submix graph to find it.
		return ParentSoundfieldSubmix->GetSubmixFormat();
	}
	else
	{
		return ISoundfieldFactory::GetFormatNameForNoEncoding();
		}
}

const USoundfieldEncodingSettingsBase* USoundfieldSubmix::GetEncodingSettings() const
{
	FName SubmixFormatName = GetSubmixFormat();

	USoundfieldSubmix* ParentSoundfieldSubmix = Cast<USoundfieldSubmix>(ParentSubmix);

	if (EncodingSettings)
	{
		return EncodingSettings;
	}
	else if (ParentSoundfieldSubmix && SoundfieldEncodingFormat == ISoundfieldFactory::GetFormatNameForInheritedEncoding())
	{
		// If this submix matches the format of whatever it's plugged into,
		// Recurse into the submix graph to match it's settings.
		return ParentSoundfieldSubmix->GetEncodingSettings();
	}
	else if (ISoundfieldFactory* Factory = ISoundfieldFactory::Get(SubmixFormatName))
	{
		// If we don't have any encoding settings, use the default.
		return Factory->GetDefaultEncodingSettings();
	}
	else
	{
		// If we don't have anything, exit.
		return nullptr;
	}
}

void USoundfieldSubmix::SanitizeLinks()
{
	bool bShouldRefreshGraph = false;

	// Iterate through children and check encoding formats.
	for (int32 Index = ChildSubmixes.Num() - 1; Index >= 0; Index--)
	{
		if (!SubmixUtils::AreSubmixFormatsCompatible(ChildSubmixes[Index], this))
		{
			CastChecked<USoundSubmixWithParentBase>(ChildSubmixes[Index])->ParentSubmix = nullptr;
			ChildSubmixes[Index]->Modify();
			ChildSubmixes.RemoveAtSwap(Index);
			bShouldRefreshGraph = true;
		}
	}

	// If this submix is now incompatible with the parent submix, disconnect it.
	if (ParentSubmix && !SubmixUtils::AreSubmixFormatsCompatible(this, ParentSubmix))
	{
		ParentSubmix->ChildSubmixes.RemoveSwap(this);
		ParentSubmix->Modify();
		ParentSubmix = nullptr;
		bShouldRefreshGraph = true;
	}

	if (bShouldRefreshGraph)
	{
#if WITH_EDITOR
		SubmixUtils::RefreshEditorForSubmix(this);
#endif
	}
}

#if WITH_EDITOR

void USoundfieldSubmix::PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent)
{
	// Whether to clean up now invalid links between submix and refresh the submix graph editor.
	bool bShouldSanitizeLinks = false;

	if (PropertyChangedEvent.Property != nullptr)
	{
		if (PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(USoundfieldSubmix, SoundfieldEncodingFormat))
		{
			bShouldSanitizeLinks = true;
			
			FAudioDeviceManager* AudioDeviceManager = FAudioDeviceManager::Get();
			if (AudioDeviceManager)
			{
				AudioDeviceManager->InitSoundSubmixes();
			}
		}
	}

	Super::PostEditChangeProperty(PropertyChangedEvent);

	if (bShouldSanitizeLinks)
	{
		SanitizeLinks();
	}
}
void UEndpointSubmix::PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent)
{	
	if (PropertyChangedEvent.Property != nullptr)
	{
		const FName PropertyName = PropertyChangedEvent.Property->GetFName();
		if (PropertyName == GET_MEMBER_NAME_CHECKED(UEndpointSubmix, EndpointType)
		|| PropertyName == GET_MEMBER_NAME_CHECKED(UEndpointSubmix, EndpointSettings)
		|| PropertyName == GET_MEMBER_NAME_CHECKED(UEndpointSubmix, EndpointSettingsClass)
		)
		{
			// Remove-re-add submix. Causes reinit with plugins.
			if (FAudioDeviceManager* AudioDeviceManager = GEngine->GetAudioDeviceManager())
			{
				AudioDeviceManager->UnregisterSoundSubmix(this);
				AudioDeviceManager->RegisterSoundSubmix(this);
			}
		}
	}
	Super::PostEditChangeProperty(PropertyChangedEvent);
}

#endif // WITH_EDITOR

void UEndpointSubmix::PostLoad()
{
	// Validate our endpoint type is enabled.
	TArray<FName> EndpointTypeNames = IAudioEndpointFactory::GetAvailableEndpointTypes();
	if (!EndpointTypeNames.Contains(EndpointType))
	{
		const FName DefaultEndpoint = IAudioEndpointFactory::GetTypeNameForDefaultEndpoint();
		UE_LOGF(LogAudio, Warning, "UEndpointSubmix [%ls] has endpoint type [%ls] which is not currently currently enabled. Changing to [%ls]",
			*GetName(), *EndpointType.ToString(), *DefaultEndpoint.ToString());
		EndpointType = DefaultEndpoint;
	}

	
	Super::PostLoad();
}

void USoundfieldSubmix::PostLoad()
{
	// Make sure the Encoding format is something we can use.
	// Fallback to something that works otherwise and warn.

	TArray<FName> FactoryNames = ISoundfieldFactory::GetAvailableSoundfieldFormats();
	if (!FactoryNames.Contains(SoundfieldEncodingFormat))
	{
		const FName NoEncoding = ISoundfieldFactory::GetFormatNameForNoEncoding();
		UE_LOGF(LogAudio, Warning, "SoundfieldSubmix [%ls] has Encoding format [%ls] which is not currently currently enabled. Changing to [%ls]",
			*GetName(), *SoundfieldEncodingFormat.ToString(), *NoEncoding.ToString());
		SoundfieldEncodingFormat = NoEncoding;
		SanitizeLinks();
	}

	Super::PostLoad();
}


IAudioEndpointFactory* UEndpointSubmix::GetAudioEndpointForSubmix() const
{
	return IAudioEndpointFactory::Get(EndpointType);
}

const UAudioEndpointSettingsBase* UEndpointSubmix::GetEndpointSettings() const
{
	return EndpointSettings;
}

ISoundfieldEndpointFactory* USoundfieldEndpointSubmix::GetSoundfieldEndpointForSubmix() const
{
	return ISoundfieldEndpointFactory::Get(SoundfieldEndpointType);
}

const USoundfieldEndpointSettingsBase* USoundfieldEndpointSubmix::GetEndpointSettings() const
{
	return EndpointSettings;
}

const USoundfieldEncodingSettingsBase* USoundfieldEndpointSubmix::GetEncodingSettings() const
{
	return EncodingSettings;
}

TArray<USoundfieldEffectBase*> USoundfieldEndpointSubmix::GetSoundfieldProcessors() const
{
	return SoundfieldEffectChain;
}


FSoundSpectrumAnalyzerSettings USoundSubmix::GetSpectrumAnalyzerSettings(EFFTSize FFTSize, EFFTPeakInterpolationMethod InterpolationMethod, EFFTWindowType WindowType, float HopSize, EAudioSpectrumType SpectrumType)
{
	FSoundSpectrumAnalyzerSettings OutSettings;

	OutSettings.FFTSize = FFTSize;
	OutSettings.WindowType = WindowType;
	OutSettings.InterpolationMethod = InterpolationMethod;
	OutSettings.SpectrumType = SpectrumType;

	const float MinHopSize = 0.001f;
	const float MaxHopSize = 10.0f;
	OutSettings.HopSize = FMath::Clamp(HopSize, MinHopSize, MaxHopSize);

	return OutSettings;
}

FSoundSpectrumAnalyzerDelegateSettings USoundSubmix::GetSpectrumAnalysisDelegateSettings(const TArray<FSoundSubmixSpectralAnalysisBandSettings>& InBandSettings, float UpdateRate, float DecibelNoiseFloor, bool bDoNormalize, bool bDoAutoRange, float AutoRangeAttackTime, float AutoRangeReleaseTime)
{
	FSoundSpectrumAnalyzerDelegateSettings DelegateSettings;

	DelegateSettings.BandSettings = InBandSettings;
	DelegateSettings.UpdateRate = UpdateRate;
	DelegateSettings.DecibelNoiseFloor = DecibelNoiseFloor;
	DelegateSettings.bDoNormalize = bDoNormalize;
	DelegateSettings.bDoAutoRange = bDoAutoRange;
	DelegateSettings.AutoRangeAttackTime = AutoRangeAttackTime;
	DelegateSettings.AutoRangeReleaseTime = AutoRangeReleaseTime;

	return DelegateSettings;
}

void USoundfieldEndpointSubmix::SanitizeLinks()
{
	bool bShouldRefreshEditor = false;

	// Iterate through children and check encoding formats.
	for (int32 Index = ChildSubmixes.Num() - 1; Index >= 0; Index--)
	{
		if (!SubmixUtils::AreSubmixFormatsCompatible(ChildSubmixes[Index], this))
		{
			CastChecked<USoundSubmixWithParentBase>(ChildSubmixes[Index])->ParentSubmix = nullptr;
			ChildSubmixes[Index]->Modify();
			ChildSubmixes.RemoveAtSwap(Index);

			bShouldRefreshEditor = true;
		}
	}
	
	if (bShouldRefreshEditor)
	{
#if WITH_EDITOR
		SubmixUtils::RefreshEditorForSubmix(this);
#endif
	}
}


void USoundfieldEndpointSubmix::PostLoad()
{
	// Validate we're set to something that's enabled.	
	TArray<FName> SoundfieldEndpointTypeNames = ISoundfieldEndpointFactory::GetAllSoundfieldEndpointTypes();
	if (!SoundfieldEndpointTypeNames.Contains(SoundfieldEndpointType))
	{
		const FName DefaultEndpoint = ISoundfieldEndpointFactory::DefaultSoundfieldEndpointName();
		UE_LOGF(LogAudio, Warning, "USoundfieldEndpointSubmix [%ls] has endpoint type [%ls] which is not currently currently enabled. Changing to [%ls]",
			*GetName(), *SoundfieldEndpointType.ToString(), *DefaultEndpoint.ToString());
		SoundfieldEndpointType = DefaultEndpoint;
		SanitizeLinks();
	}

	Super::PostLoad();	
}


#if WITH_EDITOR

void USoundfieldEndpointSubmix::PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent)
{
	if (PropertyChangedEvent.Property != nullptr)
	{
		static const FName NAME_SoundfieldFormat("SoundfieldEndpointType");

		if (PropertyChangedEvent.Property->GetFName() == NAME_SoundfieldFormat)
		{			
			// Remove-re-add submix. Causes reinit with plugins.
			if (FAudioDeviceManager* AudioDeviceManager = GEngine->GetAudioDeviceManager())
			{
				AudioDeviceManager->UnregisterSoundSubmix(this);
				AudioDeviceManager->RegisterSoundSubmix(this);
			}
			
			// Add this sound class to the parent class if it's not already added
			SanitizeLinks();
		}
	}

	Super::PostEditChangeProperty(PropertyChangedEvent);
}

#endif // WITH_EDITOR

ENGINE_API bool SubmixUtils::AreSubmixFormatsCompatible(const USoundSubmixBase* ChildSubmix, const USoundSubmixBase* ParentSubmix)
{
	if (!ChildSubmix || !ParentSubmix)
	{
		return false;
	}

	const USoundfieldSubmix* ChildSoundfieldSubmix = Cast<const USoundfieldSubmix>(ChildSubmix);

	// If both the child and parent are soundfield submixes, ensure that their formats are compatible.
	{
		const USoundfieldSubmix* ParentSoundfieldSubmix = Cast<const USoundfieldSubmix>(ParentSubmix);

		if (ChildSoundfieldSubmix && ParentSoundfieldSubmix)
		{
			ISoundfieldFactory* ChildSoundfieldFactory = ChildSoundfieldSubmix->GetSoundfieldFactoryForSubmix();
			ISoundfieldFactory* ParentSoundfieldFactory = ParentSoundfieldSubmix->GetSoundfieldFactoryForSubmix();

			if (ChildSoundfieldFactory && ParentSoundfieldFactory)
			{
				bool bCanTranscode = false;

				// To
				if (const USoundfieldEncodingSettingsBase* ParentEncodingSettings = ParentSoundfieldSubmix->GetSoundfieldEncodingSettings())
				{
					bCanTranscode |= ChildSoundfieldFactory->CanTranscodeToSoundfieldFormat(ParentSoundfieldFactory->GetSoundfieldFormatName(), *ParentEncodingSettings->GetProxy());
				}
				// From
				if (const USoundfieldEncodingSettingsBase* ChildEncodingSettings = ChildSoundfieldSubmix->GetSoundfieldEncodingSettings())
				{
					bCanTranscode |= ParentSoundfieldFactory->CanTranscodeFromSoundfieldFormat(ChildSoundfieldFactory->GetSoundfieldFormatName(), *ChildEncodingSettings->GetProxy());
				}
				
				return bCanTranscode;
			}
			else
			{
				return true;
			}
		}
	}

	// If the child is a soundfield submix and the parent is a soundfield endpoint submix, ensure that they have compatible formats.
	{
		const USoundfieldEndpointSubmix* ParentSoundfieldEndpointSubmix = Cast<const USoundfieldEndpointSubmix>(ParentSubmix);
		
		if (ChildSoundfieldSubmix && ParentSoundfieldEndpointSubmix)
		{
			ISoundfieldFactory* ChildSoundfieldFactory = ChildSoundfieldSubmix->GetSoundfieldFactoryForSubmix();
			ISoundfieldFactory* ParentSoundfieldFactory = ParentSoundfieldEndpointSubmix->GetSoundfieldEndpointForSubmix();

			if (ChildSoundfieldFactory && ParentSoundfieldFactory)
			{
				bool bCanTranscode = false;

				// TO. (Endpoint settings).
				if (const USoundfieldEncodingSettingsBase* ParentEndpointSettings = ParentSoundfieldEndpointSubmix->GetEncodingSettings())
				{
					bCanTranscode |= ChildSoundfieldFactory->CanTranscodeToSoundfieldFormat(ParentSoundfieldFactory->GetSoundfieldFormatName(), *ParentEndpointSettings->GetProxy());
				}

				// From
				if (const USoundfieldEncodingSettingsBase* ChildEncodingSettings = ChildSoundfieldSubmix->GetSoundfieldEncodingSettings())
				{
					bCanTranscode |= ParentSoundfieldFactory->CanTranscodeFromSoundfieldFormat(ChildSoundfieldFactory->GetSoundfieldFormatName(), *ChildEncodingSettings->GetProxy());
				}
				
				return bCanTranscode;
			}
			else
			{
				return true;
			}
		}
	}

	// Otherwise, these submixes are compatible.
	return true;
}

bool SubmixUtils::FindInGraph(
	const USoundSubmixBase* InEntryPoint, 
	const USoundSubmixBase* InToMatch, 
	const bool bStartFromRoot,
	FAudioDeviceHandle InDevice /*= {}*/)
{
	TSet<const USoundSubmixBase*> Visited;
	TArray<const USoundSubmixBase*> Stack;

	// Optionally ascend to the root
	const USoundSubmixBase* StartingPoint = InEntryPoint;
	if (bStartFromRoot)
	{
		StartingPoint = FindRoot(InEntryPoint, InDevice);
	}
	
	Stack.Push(StartingPoint);
	while (Stack.Num() > 0)
	{
		if (const USoundSubmixBase* Vertex = Stack.Pop())
		{
			if (Vertex == InToMatch)
			{
				return true;
			}
			else if (!Visited.Contains(Vertex))
			{
				// Unlike parents, submixes can have both dynamic and static children so search both.
				Stack.Append(Vertex->ChildSubmixes);
				if (const FDynamicChildSubmix* Dynamics = Vertex->DynamicChildSubmixes.Find(InDevice.GetDeviceID()))
				{
					Stack.Append(Dynamics->ChildSubmixes);
				}
			}
		}
	}
	return false;

}

void SubmixUtils::ForEachStaticChildRecursive(USoundSubmixBase* StartingPoint, const TFunction<void(USoundSubmixBase*)>& Op)
{
	Op(StartingPoint);
	for (TObjectPtr<USoundSubmixBase> i : StartingPoint->ChildSubmixes)
	{
		ForEachStaticChildRecursive(i,Op);
	}
}

const USoundSubmixBase* SubmixUtils::FindRoot(const USoundSubmixBase* InStartingPoint, FAudioDeviceHandle InDevice)
{	
	const USoundSubmixBase* HighestPoint = InStartingPoint;
	while (HighestPoint)
	{
		const USoundSubmixWithParentBase* WithParent = Cast<const USoundSubmixWithParentBase>(HighestPoint);
		if (!WithParent)
		{
			break;
		}

		const USoundSubmixBase* Parent = WithParent->GetParent(InDevice.GetDeviceID());
		if (!Parent)
		{
			break;
		}

		HighestPoint = Parent;
	}
	return HighestPoint;
}

#if WITH_EDITOR

ENGINE_API void SubmixUtils::RefreshEditorForSubmix(const USoundSubmixBase* InSubmix)
{
	if (!GEditor || !InSubmix)
	{
		return;
	}

	TWeakObjectPtr<USoundSubmixBase> WeakSubmix = TWeakObjectPtr<USoundSubmixBase>(const_cast<USoundSubmixBase*>(InSubmix));

	// Since we may be in the middle of a PostEditProperty call,
	// Dispatch a command to close and reopen the editor window next tick.
	AsyncTask(ENamedThreads::GameThread, [WeakSubmix]
	{
			if (WeakSubmix.IsValid())
			{
				UAssetEditorSubsystem* EditorSubsystem = GEditor->GetEditorSubsystem<UAssetEditorSubsystem>();
				TArray<IAssetEditorInstance*> SubmixEditors = EditorSubsystem->FindEditorsForAsset(WeakSubmix.Get());
				for (IAssetEditorInstance* Editor : SubmixEditors)
				{
					Editor->CloseWindow(EAssetEditorCloseReason::EditorRefreshRequested);
				}

				EditorSubsystem->OpenEditorForAsset(WeakSubmix.Get());
			}
	});
}

#endif // WITH_EDITOR

// Versioning and Deprecated Property Migration.
// --------------------------------------------

#if WITH_EDITORONLY_DATA

namespace SoundSubmixMigration
{
	// Old defaults.
	static const float OldDefault_OutputVolume(-1.0f);
	static const float OldDefault_WetLevel(-1.0f);
	static const float OldDefault_DryLevel(-1.0f);
}

void USoundSubmix::InitDeprecatedDefaults()
{
	using namespace SoundSubmixMigration;
	
	// We must init these to their old defaults prior to serialization to test if they are in fact serialized.
	OutputVolume_DEPRECATED = OldDefault_OutputVolume;
	DryLevel_DEPRECATED = OldDefault_DryLevel;
	WetLevel_DEPRECATED = OldDefault_WetLevel;
}

void USoundSubmix::HandleVersionMigration(const int32 Version)
{
	if (Version < FSoundSubmixCustomVersion::MigrateModulatedSendProperties)
	{
		using namespace SoundSubmixPrivate;
		using namespace SoundSubmixMigration;

		auto ConvertToModulatedDb = [this](const float InValue, const float InDefault, const float InDefaultModulationValue, FSoundModulationDestinationSettings& OutModulator, const TCHAR* InParamName) 
		{
			// IF after load this old property has non-default value.
			// AND the newer form is still at a default value.
			// THEN we can safely convert the value over.

			if (!FMath::IsNearlyEqual(InValue, InDefault) &&
				FMath::IsNearlyEqual(OutModulator.Value, InDefaultModulationValue))
			{
				// use -96dB as a noise floor when fixing up linear volume settings
				static constexpr float LinearNeg96dB = 0.0000158489319f;

				if (InValue <= LinearNeg96dB)
				{
					OutModulator.Value = -96.f;
				}
				else
				{
					OutModulator.Value = Audio::ConvertToDecibels(InValue);
				}

				UE_LOGF(LogAudio, Display, "SoundSubmix::HandleVersionMigration, ConvertToModulatedDb, Asset = %ls, %ls = %2.2f dB from %2.2f", *GetName(), InParamName, OutModulator.Value, InValue);
			}
		};

		// Convert.
		ConvertToModulatedDb(OutputVolume_DEPRECATED, OldDefault_OutputVolume, Default_OutputVolumeModulation, OutputVolumeModulation, TEXT("OutputVoluime"));
		ConvertToModulatedDb(WetLevel_DEPRECATED, OldDefault_WetLevel, Default_WetLevelModulation, WetLevelModulation, TEXT("WetLevel"));
		ConvertToModulatedDb(DryLevel_DEPRECATED, OldDefault_DryLevel, Default_DryLevelModulation, DryLevelModulation, TEXT("DryLevel"));
	}

	// Convert linear modulators to dB. (this has most likely happened as part of above update, but just in case we have some outliers, handle this separately).
	if (Version < FSoundSubmixCustomVersion::ConvertLinearModulatorsToDb)
	{
		auto ConvertToDb = [this](FSoundModulationDestinationSettings& OutValue, const TCHAR* InName) 
		{
			// Assume anything > 0.f is in a linear scale and convert, otherwise ignore it.
			if (OutValue.Value > 0.0f)
			{
				const float dbValue = Audio::ConvertToDecibels(OutValue.Value);
				UE_LOGF(LogAudio, Display, "SoundSubmix::HandleVersionMigration, ConvertToDb, Asset = %ls, %ls = %2.2f dB from %2.2f", *GetName(), InName, dbValue, OutValue.Value)
				OutValue.Value = dbValue;
			}
		};

		// Convert.
		ConvertToDb(OutputVolumeModulation, TEXT("OutputVolume"));
		ConvertToDb(WetLevelModulation, TEXT("WetLevel"));
		ConvertToDb(DryLevelModulation, TEXT("DryLevel"));

		OutputVolumeModulation.VersionModulators();
		WetLevelModulation.VersionModulators();
		DryLevelModulation.VersionModulators();
	}
}

#endif //WITH_EDITORONLY_DATA
