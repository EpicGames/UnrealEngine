// Copyright Epic Games, Inc. All Rights Reserved.

#include "SoundSubmixCustomVersion.h"
#include "Serialization/CustomVersion.h"

const FGuid FSoundSubmixCustomVersion::GUID(0x79D25301, 0x47AF41F8, 0xADCF10E7, 0x97415C16);

// Register the custom version with core
FCustomVersionRegistration GSoundSubmixCustomVersion(FSoundSubmixCustomVersion::GUID, FSoundSubmixCustomVersion::LatestVersion, TEXT("SoundSubmixVer"));
