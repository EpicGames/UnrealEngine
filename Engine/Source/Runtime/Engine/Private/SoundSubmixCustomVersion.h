// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/Guid.h"

#define UE_API ENGINE_API

// Custom serialization version for SoundSubmixes.
struct FSoundSubmixCustomVersion
{
	enum Type
	{
		// Before any version changes were made in the plugin
		BeforeCustomVersionWasAdded = 0,
			
		// Migrated deprecated properties OutputVolume, WetLevel, DryLevel 
		MigrateModulatedSendProperties, 

		// Convert modulated properties to dB.
		ConvertLinearModulatorsToDb,

		// -----<new versions can be added above this line>-------------------------------------------------
		VersionPlusOne,
		LatestVersion = VersionPlusOne - 1
	};

	// The GUID for this custom version number
	UE_API const static FGuid GUID;

private:
	FSoundSubmixCustomVersion() {}
};

#undef UE_API
