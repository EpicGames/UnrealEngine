// Copyright Epic Games, Inc. All Rights Reserved.
#include "Sound/SoundSubmixSend.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SoundSubmixSend)

FSoundSubmixSendInfoBase::FSoundSubmixSendInfoBase()
	: SendLevelControlMethod(ESendLevelControlMethod::Manual)
	, SoundSubmix(nullptr)
	, SendLevel(1.0f)
	, DisableManualSendClamp(false)
	, MinSendLevel(0.0f)
	, MaxSendLevel(1.0f)
	, MinSendDistance(100.0f)
	, MaxSendDistance(1000.0f)
	, bEnableLPFCutoff(0)
	, LPFCutoff(20000.0f)
	, bEnableHPFCutoff(0)
	, HPFCutoff(20.0f)
{
}

void FSoundSubmixSendInfoBase::PostSerialize(const FArchive& Ar)
{
#if WITH_EDITORONLY_DATA
	if (Ar.IsLoading())
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS

		if (LPFCutoffFrequency_DEPRECATED.IsSet() && !bEnableLPFCutoff)
		{
			bEnableLPFCutoff = 1;
			LPFCutoff = LPFCutoffFrequency_DEPRECATED.GetValue();
		}
		LPFCutoffFrequency_DEPRECATED.Reset();

		if (HPFCutoffFrequency_DEPRECATED.IsSet() && !bEnableHPFCutoff)
		{
			bEnableHPFCutoff = 1;
			HPFCutoff = HPFCutoffFrequency_DEPRECATED.GetValue();
		}
		HPFCutoffFrequency_DEPRECATED.Reset();

		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}
#endif // WITH_EDITORONLY_DATA
}

