// Copyright Epic Games, Inc. All Rights Reserved.

#include "Sound/SoundWave.h"

#include "Audio/AudioTimingLog.h"
#include "Audio/AudioTraceUtil.h"
#include "AudioCompressionSettingsUtils.h"
#include "AudioDecompress.h"
#include "AudioDerivedData.h"
#include "AudioDevice.h"
#include "Engine/Engine.h"
#include "Misc/Paths.h"
#include "Serialization/MemoryWriter.h"
#include "Sound/SoundSourceBus.h"
#include "Sound/SoundSubmix.h"
#include "Stats/StatsTrace.h"
#include "UObject/AssetRegistryTagsContext.h"
#include "UObject/FrameworkObjectVersion.h"
#include "UObject/Package.h"
#include "UObject/UObjectIterator.h"
#include "Interfaces/ITargetPlatform.h"
#include "Interfaces/ITargetPlatformManagerModule.h"
#include "Misc/App.h"
#include "Misc/ConfigCacheIni.h"
#include "SubtitleManager.h"
#include "Subtitles/ISubtitlesAndClosedCaptionsModule.h"
#include "Subtitles/SubtitlesAndClosedCaptionsDelegates.h"
#include "DerivedDataCacheInterface.h"
#include "EditorFramework/AssetImportData.h"
#include "ProfilingDebugging/CookStats.h"
#include "HAL/FileManager.h"
#include "DSP/SpectrumAnalyzer.h"
#include "DSP/EnvelopeFollower.h"
#include "Misc/OutputDeviceArchiveWrapper.h"
#include "Async/Async.h"
#include "UObject/UE5MainStreamObjectVersion.h"
#include "Algo/BinarySearch.h"
#include "Templates/UnrealTemplate.h"
#include "UObject/ObjectSaveContext.h"
#include "ISoundWaveCloudStreaming.h"
#include "SoundWaveCompiler.h"
#include "SoundFileIO/SoundFileIO.h"
#include "Serialization/MemoryReader.h"

#if WITH_EDITORONLY_DATA
#include "uewav.h"
#endif

#if WITH_EDITOR
#include "Cooker/CookEvents.h"
#include "UObject/ArchiveCookContext.h"
#endif

// WARNING: This cvar is used as part of incremental cook invalidation. Changing the name of the variable
// will invalidate all cooked sound wave packages.

#include UE_INLINE_GENERATED_CPP_BY_NAME(SoundWave)
static const TCHAR* SoundWaveDefaultLoadingBehaviorCVarName = TEXT("au.streamcache.SoundWaveDefaultLoadingBehavior");
static int32 SoundWaveDefaultLoadingBehaviorCVar = static_cast<int32>(ESoundWaveLoadingBehavior::LoadOnDemand);
FAutoConsoleVariableRef CVarSoundWaveDefaultLoadingBehavior(
	SoundWaveDefaultLoadingBehaviorCVarName,
	SoundWaveDefaultLoadingBehaviorCVar,
	TEXT("This can be set to define the default behavior when a USoundWave is loaded.\n")
	TEXT("1: Retain audio data on load, 2: prime audio data on load, 3: load on demand (No audio data is loaded until a USoundWave is played or primed)."),
	ECVF_Default);

static int32 ForceNonStreamingInEditorCVar = 0;
FAutoConsoleVariableRef CVarForceNonStreamingInEditor(
	TEXT("au.editor.ForceAudioNonStreaming"),
	ForceNonStreamingInEditorCVar,
	TEXT("When set to 1, forces any audio played to be non-streaming May force a DDC miss.\n")
	TEXT("0: Honor the Play When Silent flag, 1: stop all silent non-procedural sources."),
	ECVF_Default);

int32 DisableRetainingCVar = 0;
FAutoConsoleVariableRef CVarDisableRetaining(
	TEXT("au.streamcache.DisableRetaining"),
	DisableRetainingCVar,
	TEXT("When set to 1, USoundWaves will not retain chunks of their own audio.\n")
	TEXT("0: Don't disable retaining, 1: disable retaining."),
	ECVF_Default);

static int32 bRetainInlinedFirstChunkCVar = 0;
FAutoConsoleVariableRef CVarRetainInlinedFirstChunk(
	TEXT("au.streamcache.RetainInlinedFirstChunk"),
	bRetainInlinedFirstChunkCVar,
	TEXT("0 is off, 1 or anything else is on"),
	ECVF_Default);

static int32 BlockOnChunkLoadCompletionCVar = 0;
FAutoConsoleVariableRef CVarBlockOnChunkLoadCompletion(
	TEXT("au.streamcache.BlockOnChunkLoadCompletion"),
	BlockOnChunkLoadCompletionCVar,
	TEXT("When set to 1, USoundWaves we will always attempt to synchronously load a chunk after a USoundWave request has finished.\n")
	TEXT("0: Don't try to block after a SoundWave has completed loading a chunk, 1: Block after a USoundWave's chunk request has completed."),
	ECVF_Default);

static int32 DispatchToGameThreadOnChunkRequestCVar = 1;
FAutoConsoleVariableRef CVarDispatchToGameThreadOnChunkRequest(
	TEXT("au.streamcache.DispatchToGameThreadOnChunkRequest"),
	DispatchToGameThreadOnChunkRequestCVar,
	TEXT("When set to 1, we will always dispatch a callback to the game thread whenever a USoundWave request has finished. This may cause chunks of audio to be evicted by the time we need them.\n")
	TEXT("0: as soon as the chunk is loaded, capture the audio chunk. 1: As soon as the chunk is loaded, dispatch a callback to the gamethread."),
	ECVF_Default);

static int32 AllowReverbForMultichannelSources = 1;
FAutoConsoleVariableRef CvarAllowReverbForMultichannelSources(
	TEXT("au.AllowReverbForMultichannelSources"),
	AllowReverbForMultichannelSources,
	TEXT("Controls if we allow Reverb processing for sources with channel counts > 2.\n")
	TEXT("0: Disable, >0: Enable"),
	ECVF_Default);

#if UE_AUDIO_PROFILERTRACE_ENABLED
UE_TRACE_EVENT_BEGIN(Audio, SoundPriority)
	UE_TRACE_EVENT_FIELD(uint32, DeviceId)
	UE_TRACE_EVENT_FIELD(uint64, Timestamp)
	UE_TRACE_EVENT_FIELD(uint32, PlayOrder)
	UE_TRACE_EVENT_FIELD(uint32, ActiveSoundPlayOrder)
	UE_TRACE_EVENT_FIELD(float, Priority)
UE_TRACE_EVENT_END()

UE_TRACE_EVENT_BEGIN(Audio, SoundDistance)
	UE_TRACE_EVENT_FIELD(uint32, DeviceId)
	UE_TRACE_EVENT_FIELD(uint64, Timestamp)
	UE_TRACE_EVENT_FIELD(uint32, PlayOrder)
	UE_TRACE_EVENT_FIELD(uint32, ActiveSoundPlayOrder)
	UE_TRACE_EVENT_FIELD(float, Distance)
UE_TRACE_EVENT_END()
#endif // UE_AUDIO_PROFILERTRACE_ENABLED

inline FArchive& operator<<(FArchive& Ar, FSoundWaveCuePoint& CuePoint)
{
	FSoundWaveCuePoint::StaticStruct()->SerializeItem(Ar, &CuePoint, nullptr);
	return Ar;
}


namespace SoundWave_Private
{
	static const TCHAR* GetDefaultLoadingBehaviorCVarName()
	{
		return SoundWaveDefaultLoadingBehaviorCVarName;
	}

	static ESoundWaveLoadingBehavior GetDefaultLoadingBehaviorCVar()
	{
		// query the default loading behavior CVar
		ensureMsgf(SoundWaveDefaultLoadingBehaviorCVar >= static_cast<int32>(ESoundWaveLoadingBehavior::RetainOnLoad) &&
			SoundWaveDefaultLoadingBehaviorCVar <= static_cast<int32>(ESoundWaveLoadingBehavior::LoadOnDemand),
			TEXT("Invalid default loading behavior CVar value=%d:%s. Use value 1 (retain on load), 2 (prime on load) or 3 (load on demand)."),
			SoundWaveDefaultLoadingBehaviorCVar, EnumToString(static_cast<ESoundWaveLoadingBehavior>(SoundWaveDefaultLoadingBehaviorCVar)));

		// Clamp the value to make sure its in range.
		const ESoundWaveLoadingBehavior DefaultLoadingBehavior = static_cast<ESoundWaveLoadingBehavior>(
			FMath::Clamp<int32>(
				SoundWaveDefaultLoadingBehaviorCVar,
				static_cast<int32>(ESoundWaveLoadingBehavior::RetainOnLoad),
				static_cast<int32>(ESoundWaveLoadingBehavior::LoadOnDemand)
			));

		return DefaultLoadingBehavior;
	}
}


#if !UE_BUILD_SHIPPING
static void DumpBakedAnalysisData(const TArray<FString>& Args)
{
	if (IsInGameThread())
	{
		if (Args.Num() == 1)
		{
			const FString& SoundWaveToDump = Args[0];
			for (TObjectIterator<USoundWave> It; It; ++It)
			{
				if (It->IsTemplate(RF_ClassDefaultObject))
				{
					continue;
				}

				if (SoundWaveToDump.Equals(It->GetName()))
				{
#if WITH_EDITOR
					It->LogBakedData();
#endif // WITH_EDITOR
				}
			}
		}
	}
}

static FAutoConsoleCommand DumpBakedAnalysisDataCmd(
	TEXT("au.DumpBakedAnalysisData"),
	TEXT("debug command to dump the baked analysis data of a sound wave to a csv file."),
	FConsoleCommandWithArgsDelegate::CreateStatic(&DumpBakedAnalysisData)
);
#endif

#if ENABLE_COOK_STATS
namespace SoundWaveCookStats
{
	static FCookStats::FDDCResourceUsageStats UsageStats;
	static FCookStatsManager::FAutoRegisterCallback RegisterCookStats([](FCookStatsManager::AddStatFuncRef AddStat)
	{
		UsageStats.LogStats(AddStat, TEXT("SoundWave.Usage"), TEXT(""));
	});
}
#endif

FSoundWaveData::FSoundWaveData(const USoundWave& InWave) 
	: bIsLooping(0)
	, bIsTemplate(0)
	, bIsStreaming(0)
	, bIsSeekable(0)
	, bShouldUseStreamCaching(0)
	, bLoadingBehaviorOverridden(0)
	, bHasError(0)
	, bHasSubtitleAssetUserData(0)
#if WITH_EDITOR
	, bLoadedFromCookedData(0)
#endif //WITH_EDITOR
{
	InitializeDataFromSoundWave(InWave);
}

FSoundWaveData::~FSoundWaveData()
{
	ZerothChunkData.Empty();
	ResourceData.Empty();
}

void FSoundWaveData::CacheRuntimeFormatDependentState(const FName InFormatName)
{
	// FindRuntimeFormat returns NAME_None for templates / dedicated servers / !FApp::CanEverRenderAudio() contexts
	if (!InFormatName.IsNone())
	{
		// Save format.
		RuntimeFormat = InFormatName;

		// Query the factory for the decoder.
		// Note: not all decoder plugins are loaded in every context (e.g. commandlets, dedicated servers).
		// Only ensure in contexts where the full audio system is expected to be running.
		const IAudioInfoFactory* Factory = IAudioInfoFactoryRegistry::Get().Find(InFormatName);
		ensureMsgf(Factory || IsRunningCommandlet() || IsRunningDedicatedServer() || !FApp::CanEverRenderAudio(), TEXT("Decoder for AudioFormat '%s' not found"), *InFormatName.ToString());
		if (Factory)
		{
			bIsSeekable = Factory->IsSeekingSupported(bIsStreaming);	// bStreaming should be set before calling this.
		}
	}
}

void FSoundWaveData::InitializeDataFromSoundWave(const USoundWave& InWave)
{
	LLM_SCOPE(ELLMTag::AudioSoundWaves);

	NameCached = InWave.GetFName();

	// cache package name
	if (UPackage* Package = InWave.GetPackage())
	{
		PackageNameCached = Package->GetFName();
	}
	
	SoundWaveKeyCached = FObjectKey(&InWave);
	SampleRate = InWave.GetSampleRateForCurrentPlatform();
	Duration = InWave.Duration;
	NumChannels = InWave.NumChannels;

	NumFrames = (int32)(Duration * (float)SampleRate);
	WaveGuid = InWave.CompressedDataGuid;

	// update shared flags
	bIsStreaming = InWave.IsStreaming();
	bIsLooping = InWave.IsLooping();
	bIsTemplate = InWave.IsTemplate();
	bIsSeekable = false; // This is determined by the decoder in CacheRuntimeFormatDependentState.
	SoundAssetCompressionType = InWave.GetSoundAssetCompressionType();
	bShouldUseStreamCaching = InWave.ShouldUseStreamCaching();
	bHasSubtitleAssetUserData = const_cast<USoundWave&>(InWave).HasAssetUserDataOfClass(USubtitleAssetUserData::StaticClass());

	// Cache the decoder and any dependent state.
	const FName FoundFormat = FindRuntimeFormat(InWave);
	CacheRuntimeFormatDependentState(FoundFormat);

#if WITH_EDITOR
	bLoadedFromCookedData = InWave.IsLoadedFromCookedData();
	CurrentChunkRevision = InWave.CurrentChunkRevision;

	// only necessary to set cue points here in editor 
	// when the sample rate can change due to platform settings changing
	// or during first import of the asset.
	// otherwise, CuePoints are always updated during serialization
	SetAllCuePoints(InWave.GetCuePointsScaledForSampleRate(SampleRate));
#endif //WITH_EDITOR
}

void FSoundWaveData::OverrideRuntimeFormat(const FName& InRuntimeFormat)
{
	CacheRuntimeFormatDependentState(InRuntimeFormat);
}

static FName GetBaseFormatFromSuffixedFormat(const FName& InFormat)
{
	// If the format contains a delimiter, it's using a platform cook overrides with a suffix.
	// Return up to the first delimiter e.g. "ADPCM_some_suffix" -> "ADPCM"
	
	// It's not great that we store the entire suffixed format in FName as this heavily pollutes
	// the name table and never gets freed.
	// e.g. suffixed name: BINKA_SCVER_5028_R4DV_false_SR0_48000.000000_SR1_32000.000000_SR2_24000.000000_SR3_12000.000000_SR4_8000.000000_QMOD_1.000000_CQLT_-1_ASTH_0.000000_INLC_false_LCK1_0.000000_CSZE_65536_LCF_false_ZCS_256_MCSO_-1_END
	TStringBuilder<FName::StringBufferSize> Name;
	Name << InFormat;

	const TCHAR* DelimAt = FCString::Strchr(Name.ToString(), TEXT('_'));
	if (DelimAt)
	{
		TStringView BaseFormat(Name.ToString(), DelimAt - Name.ToString());

		// We know this FName already exists in the name table because it started out as one before the suffixes.
		return FName(BaseFormat);
	}
	return InFormat;
}

FName FSoundWaveData::FindRuntimeFormat(const USoundWave& InWave) const
{	
#if WITH_EDITOR	
	// If this is an editor build, we can ask ITargetPlatform.
	if (!InWave.IsLoadedFromCookedData())
	{
		if (ITargetPlatformManagerModule* TPM = GetTargetPlatformManager())
		{
			if (ITargetPlatform* RunningTarget = TPM->GetRunningTargetPlatform())
			{
				return RunningTarget->GetWaveFormat(&InWave);
			}
		}
	}
#endif //WITH_EDITOR

	// If we can't render audio, we won't contain any compressed data.
	if (InWave.IsTemplate() || !FApp::CanEverRenderAudio())
	{
		return {};
	}

	// If the data is "forceinline" it will exist in the CompressedFormatData.
	if (LoadingBehavior == ESoundWaveLoadingBehavior::ForceInline)
	{
		TArray<FName> ContainedFormats;
		CompressedFormatData.GetContainedFormats(ContainedFormats);

		// Only one format is supported currently.
		if (ensureMsgf(ContainedFormats.Num() == 1, TEXT("ContainedFormats::Num()=%d"), ContainedFormats.Num()))
		{
			return GetBaseFormatFromSuffixedFormat(ContainedFormats[0]);
		}
		
		// Fail.
		return {};
	}
	// ... else the audio format is written in RunningPlatformData (i.e. stream cache).
	else
	{
		return RunningPlatformData.GetAudioFormat();
	}
}

uint32 FSoundWaveData::GetNumChunks() const
{
	// This function shouldn't be called on audio marked "ForceInline."
	ensureMsgf(LoadingBehavior != ESoundWaveLoadingBehavior::ForceInline, TEXT("Calling GetNumChunks without RunningPlatformData is not allowed! SoundWave: %s - %s")
		, *GetFName().ToString(), EnumToString(LoadingBehavior));

	return RunningPlatformData.GetNumChunks();
}

void FSoundWaveData::DiscardZerothChunkData()
{
#if WITH_EDITORONLY_DATA
	// Make sure that no async tasks are running before discarding the zeroth chunk
	RunningPlatformData.FinishCache();
#endif

	ZerothChunkData.Empty();
}

void FSoundWaveData::SetAllCuePoints(const TArray<FSoundWaveCuePoint>& InCuePoints)
{
	CuePoints.Reset();
	LoopRegions.Reset();

	for (const FSoundWaveCuePoint& CuePoint : InCuePoints)
	{
		if (!CuePoint.IsLoopRegion())
		{
			CuePoints.Add(CuePoint);
		}
		else
		{
			LoopRegions.Add(CuePoint);
		}
	}
}

uint32 FSoundWaveData::GetSizeOfChunk(uint32 ChunkIndex) const
{
	check(ChunkIndex < GetNumChunks());

	// This function shouldn't be called on audio marked "ForceInline."
	ensureMsgf(RunningPlatformData.GetNumChunks()
		, TEXT("Calling GetNumChunks() without RunningPlatformData is not allowed! SoundWave: %s - %s")
		, *GetFName().ToString()
		, EnumToString(LoadingBehavior));

	check((ChunkIndex < (uint32)GetNumChunks()));
	return RunningPlatformData.GetChunks()[ChunkIndex].AudioDataSize;
}


void FSoundWaveData::ReleaseCompressedAudio() const
{
	LLM_SCOPE(ELLMTag::AudioSoundWaves);

	FirstChunk = FAudioChunkHandle();
}

const TArray<FSoundWaveCuePoint> FSoundWaveData::GetCuePointsAndLoopRegions() const
{
	TArray<FSoundWaveCuePoint> CuePointsAndLoopRegions;
	CuePointsAndLoopRegions.Reserve(GetLoopRegions().Num() + GetCuePoints().Num());

	// Previously loops had two cue points (one marked as a cue point, the other a loop region),
	// now they only have one that is marked as a loop region so we need to append Loop Regions.
	// Append must happen in this order to maintain the label for loops that were imported before the fix because
	// meta sounds will pick only one cue point to render if they have matching frames.
	CuePointsAndLoopRegions.Append(GetLoopRegions());
	CuePointsAndLoopRegions.Append(GetCuePoints());
	
	return CuePointsAndLoopRegions;
}

TArrayView<const uint8> FSoundWaveData::GetZerothChunk(bool bForImmediatePlayback) const
{
	if (GIsEditor)
	{
		LoadZerothChunk();
	}

	if (IsZerothChunkDataLoaded())
	{
		if (ShouldUseStreamCaching())
		{
			if (GetNumChunks() > 1)
			{
				// Prime first chunk for playback
				IStreamingManager::Get().GetAudioStreamingManager().RequestChunk(this->AsShared(), 1, [](EAudioChunkLoadResult InResult) {}, ENamedThreads::AnyThread, bForImmediatePlayback);
			}

			return GetZerothChunkDataView();
		}
		else
		{
			FAudioChunkHandle ChunkHandle = IStreamingManager::Get().GetAudioStreamingManager().GetLoadedChunk(this->AsShared(), 0);
			return TArrayView<const uint8>(ChunkHandle.GetData(), ChunkHandle.Num());
		}
	}
	
	return {};
}

bool FSoundWaveData::HasCompressedData(FName Format, ITargetPlatform* TargetPlatform) const
{
	if (IsTemplate() || IsRunningDedicatedServer())
	{
		return false;
	}

	const FPlatformAudioCookOverrides* CompressionOverrides = nullptr;

	if (GIsEditor)
	{
		if (TargetPlatform)
		{
			CompressionOverrides = FPlatformCompressionUtilities::GetCookOverrides(*TargetPlatform->IniPlatformName());
		}
	}
	else
	{
		// TargetPlatform is not available on consoles/mobile, so we have to grab it ourselves:
		CompressionOverrides = FPlatformCompressionUtilities::GetCookOverrides();
	}

	if (CompressionOverrides)
	{
#if WITH_EDITOR
		FName PlatformSpecificFormat;
		FString HashedString = *Format.ToString();
		FPlatformAudioCookOverrides::GetHashSuffix(CompressionOverrides, HashedString);
		PlatformSpecificFormat = *HashedString;
#else
		// on non-editor builds, we cache the concatenated format in a static FName.
		static FName PlatformSpecificFormat;
		static FName CachedFormat;
		if (!Format.IsEqual(CachedFormat))
		{
			FString HashedString = *Format.ToString();
			FPlatformAudioCookOverrides::GetHashSuffix(CompressionOverrides, HashedString);
			PlatformSpecificFormat = *HashedString;

			CachedFormat = Format;
		}
#endif // WITH_EDITOR
		return CompressedFormatData.Contains(PlatformSpecificFormat);
	}
	else
	{
		return CompressedFormatData.Contains(Format);
	}
}

void FSoundWaveData::SetError(const TCHAR* InErrorMsg)
{
	UE_CLOGF(!bHasError, LogAudio, Warning, "Flagging SoundWave '%ls' to have errors, preventing playback. ErrorMsg='%ls'", 
		*GetPackageName().ToString(),
		InErrorMsg ? InErrorMsg : TEXT("")
	);
	bHasError = true;
}

bool FSoundWaveData::HasError() const
{
	return bHasError;
}

bool FSoundWaveData::ResetError()
{
	bool bHadError = bHasError;
	bHasError = false;
	return bHadError;
}

FByteBulkData* FSoundWaveData::GetCompressedData(FName Format, const FPlatformAudioCookOverrides* CompressionOverrides) const
{
	if (IsTemplate() || IsRunningDedicatedServer())
	{
		return nullptr;
	}

	TRACE_CPUPROFILER_EVENT_SCOPE(FSoundWaveProxy::GetCompressedData);
	FName PlatformSpecificFormat = USoundWave::GetPlatformSpecificFormat(Format, CompressionOverrides);

	bool bContainedValidData = CompressedFormatData.Contains(PlatformSpecificFormat);
	if (!bContainedValidData)
	{
		// the proxy does not support interaction w/ FDerivedAudioDataCompressor
		return nullptr;
	}

	FByteBulkData* Result = &CompressedFormatData.GetFormat(PlatformSpecificFormat);
	check(Result);
	return Result->GetBulkDataSize() > 0 ? Result : NULL; // we don't return empty bulk data...but we save it to avoid thrashing the DDC
}

bool FSoundWaveData::GetChunkData(int32 ChunkIndex, uint8** OutChunkData, bool bMakeSureChunkIsLoaded) const
{
	LLM_SCOPE(ELLMTag::AudioSoundWaves);
	if (GetChunkFromDDC(ChunkIndex, OutChunkData, bMakeSureChunkIsLoaded) == 0)
	{
#if WITH_EDITORONLY_DATA
		// Unable to load chunks from the cache. Rebuild the sound and attempt to precache it.
		UE_LOGF(LogAudio, Display, "GetChunkData failed, rebuilding %ls", *GetPackageName().ToString());

		//		ForceRebuildPlatformData();
		if (GetChunkFromDDC(ChunkIndex, OutChunkData, bMakeSureChunkIsLoaded) == 0)
		{
			UE_LOGF(LogAudio, Warning, "Failed to build sound %ls.", *GetPackageName().ToString());
		}
		else
		{
			// Succeeded after rebuilding platform data
			return true;
		}
#else
		// Failed to find the SoundWave chunk in the cooked package.
		UE_LOGF(LogAudio, Warning, "GetChunkData failed while streaming. Ensure the following package is cooked: %ls", *GetPackageName().ToString());
#endif // #if WITH_EDITORONLY_DATA
		return false;
	}
	return true;
}

bool FSoundWaveData::IsZerothChunkDataLoaded() const
{
	return GetZerothChunkData().GetView().Num() > 0;
}

FBulkDataBuffer<uint8>& FSoundWaveData::GetZerothChunkData() const
{
	// Async loading only exists in editor builds
	FSoundWaveData* MutableThis = const_cast<FSoundWaveData*>(this);

#if WITH_EDITOR
	MutableThis->LoadZerothChunk();
#endif // #if WITH_EDITOR

	return MutableThis->ZerothChunkData;
}

const TArrayView<uint8> FSoundWaveData::GetZerothChunkDataView() const
{
	FBulkDataBuffer<uint8>::ViewType View = GetZerothChunkData().GetView();
	check(View.Num() <= MAX_int32);
	return TArrayView<uint8>(View.GetData(), static_cast<int32>(View.Num()));
}

bool FSoundWaveData::HasChunkSeekTable(int32 InChunkIndex) const
{
	const TIndirectArray<FStreamedAudioChunk>& Chunks = RunningPlatformData.GetChunks();
	if (Chunks.IsValidIndex(InChunkIndex))
	{
		return Chunks[InChunkIndex].SeekOffsetInAudioFrames != INDEX_NONE;
	}
	return false;
}

int32 FSoundWaveData::FindChunkIndexForSeeking(uint32 InTimeInAudioFrames) const
{
	const TIndirectArray<FStreamedAudioChunk>& Chunks = RunningPlatformData.GetChunks();
	int32 NumChunks = Chunks.Num();

	// Find most in range chunk.
	int32 InRangeChunk = INDEX_NONE;
	for (int32 i=0; i < NumChunks; ++i)
	{
		// Ignore chunks without tables. (should only be chunk0).
		if (Chunks[i].SeekOffsetInAudioFrames == INDEX_NONE)
		{
			continue;
		}

		// If its out of bound, use the last valid one.
		if (Chunks[i].SeekOffsetInAudioFrames > InTimeInAudioFrames)
		{
			return InRangeChunk;
		}

		// In bounds and valid... 
		InRangeChunk = i;
	}
	return InRangeChunk;
}

bool FSoundWaveData::LoadZerothChunk() const
{
#if WITH_EDITOR
	TRACE_CPUPROFILER_EVENT_SCOPE(FSoundWaveData::LoadZerothChunk);

	LLM_SCOPE(ELLMTag::AudioSoundWaves);

	if (!ShouldUseStreamCaching())
	{
		return false;
	}

	{
		FScopeLock ZerothChunkDataScopeLock(&LoadZerothChunkDataCriticalSection);

		if (ZerothChunkData.GetView().Num() == 0)
		{
			if (!bLoadedFromCookedData)
			{
				// In editor, we wait until the zeroth chunk is required to force finish compilation
				// and retrieve it from DDC.  The rational is that async compilation will either prefetch
				// chunks or rebuild them if unavailable so that this DDC request will be fulfilled from local cache.
				uint8* TempChunkBuffer = nullptr;
				int32 ChunkSizeInBytes = 0;

				// GetNumChunks will force finish compilation if an async task still exists.
				// Do not query chunk from DDC if no chunk exists as it would ensure inside GetChunkFromDDC.
				if (RunningPlatformData.GetNumChunks() > 0)
				{
					ChunkSizeInBytes = GetChunkFromDDC(0, &TempChunkBuffer, true);
				}

				// Since we block for the DDC in the previous call we should always have the chunk loaded.
				if (ChunkSizeInBytes != 0)
				{
					ZerothChunkData.Reset(TempChunkBuffer, ChunkSizeInBytes);
				}
				else
				{
					ZerothChunkData.Empty();
					UE_LOGF(LogAudio, Warning, "FSoundWaveData::LoadZerothChunk: Unsuccessful load of zeroth chunk from DDC. Asset requires manual re-cook.\n\tAsset: '%ls'\n\tDerivedDataKey: '%ls'",
						*GetFName().ToString(),
						*RunningPlatformData.DerivedDataKey);
					return false;
				}
			}
			else
			{
				// The zeroth chunk is cooked out to RunningPlatformData, so retrieve it.
				if (ensureMsgf(GetNumChunks() > 0, TEXT("SoundWave '%s' is cooked but contains no chunks."), *GetFName().ToString()))
				{
					FStreamedAudioChunk& ZerothChunk = GetChunk(0);

					// Sanity check to ensure bulk size is set up
					UE_CLOGF(ZerothChunk.BulkData.GetBulkDataSize() != ZerothChunk.DataSize, LogAudio, Warning
						, "SoundWave '%ls' bulk data serialized out had a mismatched size with the DataSize field."
							"\nBulk Data Reported Size: %d"
							"\nBulk Data Actual Size: %lld"
						, *GetFName().ToString(), ZerothChunk.DataSize, ZerothChunk.BulkData.GetBulkDataSize());

					ZerothChunkData = ZerothChunk.BulkData.GetCopyAsBuffer(ZerothChunk.AudioDataSize, true);
				}
				else
				{
					// failed as we have no chunks.
					return false;
				}
			}
		}
	}
#endif

	return true;
}

#if WITH_EDITOR
int32 FSoundWaveData::GetCurrentChunkRevision() const
{
	return CurrentChunkRevision;
}
#endif // #if WITH_EDITOR

FStreamedAudioChunk& FSoundWaveData::GetChunk(uint32 ChunkIndex) const
{
	// This function shouldn't be called on audio marked "ForceInline."
	ensureMsgf(RunningPlatformData.GetNumChunks()
		, TEXT("Calling GetNumChunks() without RunningPlatformData is not allowed! SoundWave: %s - %s")
		, *GetFName().ToString()
		, EnumToString(LoadingBehavior));

	check((ChunkIndex < (uint32)GetNumChunks()));
	return RunningPlatformData.GetChunks()[ChunkIndex];
}

int32 FSoundWaveData::GetChunkFromDDC(int32 ChunkIndex, uint8** OutChunkData, bool bMakeSureChunkIsLoaded) const
{
	LLM_SCOPE(ELLMTag::AudioSoundWaves);
	// This function shouldn't be called on audio marked "ForceInline."
	ensureMsgf(RunningPlatformData.GetNumChunks()
		, TEXT("Calling GetNumChunks on a FSoundWaveProxy without RunningPlatformData is not allowed! SoundWave: %s - %s")
		, *GetFName().ToString()
		, EnumToString(LoadingBehavior));

	return RunningPlatformData.GetChunkFromDDC(ChunkIndex, OutChunkData, bMakeSureChunkIsLoaded);
}

#if WITH_EDITORONLY_DATA
FString FSoundWaveData::GetDerivedDataKey() const
{
	check(RunningPlatformData.GetNumChunks());

	// This function shouldn't be called on audio marked "ForceInline."
	ensureMsgf(RunningPlatformData.GetNumChunks()
		, TEXT("Calling GetNumChunks on a FSoundWaveProxy without RunningPlatformData is not allowed! SoundWave: %s - %s")
		, *GetFName().ToString()
		, EnumToString(LoadingBehavior));

	return RunningPlatformData.DerivedDataKey;
}
#endif // #if WITH_EDITORONLY_DATA


#if WITH_EDITOR

/** IInterface_AsyncCompilation implementation begin*/
bool USoundWave::IsCompiling() const
{
	return GetWorkingSoundWaveData()->RunningPlatformData.IsCompiling();
}
/** IInterface_AsyncCompilation implementation end*/

bool USoundWave::IsAsyncWorkComplete() const
{
	return GetWorkingSoundWaveData()->RunningPlatformData.IsAsyncWorkComplete();
}

void USoundWave::PostImport()
{
	GetWorkingSoundWaveData()->InitializeDataFromSoundWave(*this);
}

bool USoundWave::RescheduleAsyncTask(FQueuedThreadPool* InThreadPool, EQueuedWorkPriority InPriority)
{
	return GetWorkingSoundWaveData()->RunningPlatformData.RescheduleAsyncTask(InThreadPool, InPriority);
}

bool USoundWave::WaitAsyncTaskWithTimeout(float InTimeoutInSeconds)
{
	return GetWorkingSoundWaveData()->RunningPlatformData.WaitAsyncTaskWithTimeout(InTimeoutInSeconds);
}

#endif // #if WITH_EDITOR

int32 USoundWave::GetResourceSize() const
{
	return GetWorkingSoundWaveData()->ResourceSize;
}

const uint8* USoundWave::GetResourceData() const
{
	return GetWorkingSoundWaveData()->GetResourceData();
}

#if WITH_EDITORONLY_DATA
void USoundWave::SetCloudStreamingEnabled(bool bEnable)
{
	bEnableCloudStreaming = bEnable;
}

bool USoundWave::IsCloudStreamingEnabled() const
{
	return !!bEnableCloudStreaming;
}

void USoundWave::TriggerRecookForCloudStreaming()
{
	if (IsCloudStreamingEnabled())
	{
		InvalidateCompressedData();	// this will reinitialize the compressed data GUID and hence make sure we do generate new DDC entries at all times
		MarkPackageDirty();			// also mark the package as dirty to ensure we trigger a cook
	}
}
#endif // #if WITH_EDITORONLY_DATA

ITargetPlatform* USoundWave::GetRunningPlatform()
{
	ITargetPlatformManagerModule* TPM = GetTargetPlatformManager();
	if (TPM)
	{
		return TPM->GetRunningTargetPlatform();
	}
	else
	{
		return nullptr;
	}
}

ENGINE_API const TCHAR* USoundWave::GetDefaultLoadingBehaviorCVarName()
{
	return SoundWave_Private::GetDefaultLoadingBehaviorCVarName();
}

ENGINE_API ESoundWaveLoadingBehavior USoundWave::GetDefaultLoadingBehavior() 
{
	return SoundWave_Private::GetDefaultLoadingBehaviorCVar();
}

/*-----------------------------------------------------------------------------
	FStreamedAudioChunk
-----------------------------------------------------------------------------*/

void FStreamedAudioChunk::Serialize(FArchive& Ar, UObject* Owner, int32 ChunkIndex)
{
	DECLARE_SCOPE_CYCLE_COUNTER( TEXT("FStreamedAudioChunk::Serialize"), STAT_StreamedAudioChunk_Serialize, STATGROUP_LoadTime );

	enum 
	{
		IsCooked			 = 1 << 0,
		HasSeekOffset		 = 1 << 1,
		IsInlined			 = 1 << 2,
	};
	
	// Bit-pack flags into a single uint32, instead of a bool.
	uint32 Flags = 0;
	if (Ar.IsSaving())
	{
#if WITH_EDITORONLY_DATA
		// Chunk 0 is always inlined, others are optionally inlined.
		const bool bShouldInline = ChunkIndex == 0 || bInlineChunk;
		Flags |= bShouldInline ? IsInlined : 0;
#endif //WITH_EDITORONLY_DATA
		Flags |= Ar.IsCooking() ? IsCooked : 0;
		Flags |= SeekOffsetInAudioFrames != INDEX_NONE ? HasSeekOffset : 0;
	}

	Ar << Flags;

	if (Ar.IsSaving())
	{
		if (Flags & IsInlined)
		{
			BulkData.SetBulkDataFlags(BULKDATA_ForceInlinePayload);
		}
		else
		{
			BulkData.SetBulkDataFlags(BULKDATA_Force_NOT_InlinePayload);
		}
	}

	// streaming doesn't use memory mapped IO
	BulkData.Serialize(Ar, Owner, ChunkIndex, false);
	Ar << DataSize;
	Ar << AudioDataSize;

	if (Ar.IsLoading() )
	{
		// Sanity check these look sane. 
		// And ensure reset in the case they are bad.
		if (!ensure(DataSize > 0))
		{
			DataSize = 0;
		}
		if (!ensure(AudioDataSize > 0))
		{
			AudioDataSize = 0;
		}
		if (!ensure(AudioDataSize <= DataSize))
		{
			AudioDataSize = DataSize;
		}
	}

	// To save bloat, only Serialize the seek offset if this chunk uses it. 
	// Otherwise this defaults to INDEX_NONE
	if (Flags & HasSeekOffset)
	{
		Ar << SeekOffsetInAudioFrames;
	}

#if WITH_EDITORONLY_DATA
	if (!(Flags & IsCooked))
	{
		Ar << DerivedDataKey;
	}

	if (Ar.IsLoading())
	{
		bLoadedFromCookedPackage = !!(Flags & IsCooked);
		bInlineChunk = !!(Flags & IsInlined);
	}
#endif // #if WITH_EDITORONLY_DATA
}

bool FStreamedAudioChunk::GetCopy(void** OutChunkData)
{
	if (AudioDataSize <= 0 || DataSize <= 0)
	{
		return false;
	}
	
	if (!CachedDataPtr)
	{
		if (AudioDataSize != DataSize)
		{
			TArray<uint8> TempChunkBuffer;
			TempChunkBuffer.AddUninitialized(DataSize);
			void* DataDestPtr = TempChunkBuffer.GetData();
			BulkData.GetCopy(&DataDestPtr, true);

			FMemory::Memcpy(*OutChunkData, TempChunkBuffer.GetData(), AudioDataSize);
		}
		else
		{
			BulkData.GetCopy((void**)&CachedDataPtr, true);
		}
	}

	if (CachedDataPtr)
	{
		FMemory::Memcpy(*OutChunkData, CachedDataPtr, DataSize);
		return true;
	}

	return false;
}

FBulkDataBuffer<uint8> FStreamedAudioChunk::MoveOutAsBuffer() 
{
	if (ensure(AudioDataSize > 0))
	{
		if (ensure(AudioDataSize <= BulkData.GetElementCount()))
		{
			const int64 RequestedSize = FMath::Min(AudioDataSize, BulkData.GetElementCount());
			return BulkData.GetCopyAsBuffer(RequestedSize, /* Discard Internal Copy */ true);
		}
	}
	return {};
}

#if WITH_EDITORONLY_DATA
uint32 FStreamedAudioChunk::StoreInDerivedDataCache(const FString& InDerivedDataKey, const FStringView& SoundWaveName)
{
	int32 BulkDataSizeInBytes = BulkData.GetBulkDataSize();
	check(BulkDataSizeInBytes > 0);

	TArray<uint8> DerivedData;
	FMemoryWriter Ar(DerivedData, /*bIsPersistent=*/ true);
	Ar << BulkDataSizeInBytes;
	Ar << AudioDataSize;

	{
		void* BulkChunkData = BulkData.Lock(LOCK_READ_ONLY);
		Ar.Serialize(BulkChunkData, BulkDataSizeInBytes);
		BulkData.Unlock();
	}

	const uint32 Result = DerivedData.Num();
	GetDerivedDataCacheRef().Put(*InDerivedDataKey, DerivedData, SoundWaveName);
	DerivedDataKey = InDerivedDataKey;
	BulkData.RemoveBulkData();
	return Result;
}
#endif // #if WITH_EDITORONLY_DATA

USoundWave::USoundWave(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
#if WITH_EDITORONLY_DATA
	, RawData(this)
#endif
{
	Volume = 1.0;
	Pitch = 1.0;
	SubtitlePriority = DEFAULT_SUBTITLE_PRIORITY;
	ResourceState = ESoundWaveResourceState::NeedsFree;
	RawPCMDataSize = 0;
	SetPrecacheState(ESoundWavePrecacheState::NotStarted);

	FrequenciesToAnalyze.Add(100.0f);
	FrequenciesToAnalyze.Add(500.0f);
	FrequenciesToAnalyze.Add(1000.0f);
	FrequenciesToAnalyze.Add(5000.0f);

#if WITH_EDITORONLY_DATA
	FFTSize = ESoundWaveFFTSize::Medium_512;
	FFTAnalysisFrameSize = 1024;
	FFTAnalysisAttackTime = 10;
	FFTAnalysisReleaseTime = 3000;
	EnvelopeFollowerFrameSize = 1024;
	EnvelopeFollowerAttackTime = 10;
	EnvelopeFollowerReleaseTime = 100;
	bEnableCloudStreaming = false;

	CuePointOrigin = ESoundWaveCuePointOrigin::MarkerTransformation;
#endif

	bCachedSampleRateFromPlatformSettings = false;
	bSampleRateManuallyReset = false;
	CachedSampleRateOverride = 0.0f;

	bProcedural = false;
	bRequiresStopFade = false;

	SoundAssetCompressionType = ESoundAssetCompressionType::ProjectDefined;
	CompressionQuality = 80;

#if WITH_EDITOR
	bLoadedFromCookedData = false;

	GetWorkingSoundWaveData()->ResourceData.Empty();
#endif
}

void USoundWave::GetResourceSizeEx(FResourceSizeEx& CumulativeResourceSize)
{
	Super::GetResourceSizeEx(CumulativeResourceSize);

	if (!GEngine)
	{
		return;
	}

	// First, add any UProperties that are on the USoundwave itself:
	CumulativeResourceSize.AddDedicatedSystemMemoryBytes(sizeof(USoundWave));

	// Add all cooked spectral and envelope data:
	CumulativeResourceSize.AddDedicatedSystemMemoryBytes(FrequenciesToAnalyze.Num() * sizeof(float));
	CumulativeResourceSize.AddDedicatedSystemMemoryBytes(CookedSpectralTimeData.Num() * sizeof(FSoundWaveSpectralTimeData));

	for (FSoundWaveSpectralTimeData& Entry : CookedSpectralTimeData)
	{
		CumulativeResourceSize.AddDedicatedSystemMemoryBytes(Entry.Data.Num() * sizeof(FSoundWaveSpectralDataEntry));
	}

	CumulativeResourceSize.AddDedicatedSystemMemoryBytes(CookedEnvelopeTimeData.Num() * sizeof(FSoundWaveEnvelopeTimeData));

	// Add zeroth chunk data, if it's used (if this USoundWave isn't streaming,
	// or the bulk data is culled on this platform, this won't report).
	if(FApp::CanEverRenderAudio())
	{
		CumulativeResourceSize.AddDedicatedSystemMemoryBytes(GetWorkingSoundWaveData()->GetZerothChunkData().GetView().Num());
	}

	// Finally, report the actual audio memory being used, if this asset isn't using the stream cache.
	if (FAudioDevice* LocalAudioDevice = GEngine->GetMainAudioDeviceRaw())
	{
		if (DecompressionType == DTYPE_RealTime && CachedRealtimeFirstBuffer)
		{
			CumulativeResourceSize.AddDedicatedSystemMemoryBytes(MONO_PCM_BUFFER_SIZE * NumChannels);
		}

		if (!FPlatformProperties::SupportsAudioStreaming() || !IsStreaming(nullptr))
		{
			CumulativeResourceSize.AddDedicatedSystemMemoryBytes(GetCompressedDataSize(GetRuntimeFormat()));
		}
	}
}


#if WITH_EDITORONLY_DATA

void USoundWave::SetTimecodeInfo(const FSoundWaveTimecodeInfo& InTimecode)
{
	// Keep editor info on the SoundWave.
	TimecodeInfo = InTimecode;

	// Set SoundBase offset.
	FSoundTimecodeOffset Offset;
	Offset.NumOfSecondsSinceMidnight = InTimecode.GetNumSecondsSinceMidnight();
	SetTimecodeOffset(Offset);
}

TOptional<FSoundWaveTimecodeInfo> USoundWave::GetTimecodeInfo() const
{
	// Do we have any time-code info that's not the defaults?
	static const FSoundWaveTimecodeInfo Defaults;
	if(Defaults == TimecodeInfo)
	{
		return {};
	}
	return TimecodeInfo;
}

bool USoundWave::FixTimeCodeInfo()
{
	if (TimecodeInfo.OriginatorDate.IsEmpty() && TimecodeInfo.Description.IsEmpty() && TimecodeInfo.OriginatorReference.IsEmpty()
		&& TimecodeInfo.OriginatorDescription.IsEmpty() && TimecodeInfo.OriginatorTime.IsEmpty())
	{
		// TimecodeInfo data is empty and does not need to be fixed
		return false;
	}

	FSharedBuffer BulkDataBuffer;
	BulkDataBuffer = RawData.GetPayload().Get();

	FWaveModInfo WaveInfo;
	if (!WaveInfo.ReadWaveInfo((const uint8*)BulkDataBuffer.GetData(), BulkDataBuffer.GetSize()))
	{
		UE_LOGF(LogAudio, Error, "Sound wave payload in asset %ls failed to parse as a wave in FixTimeCodeInfo!", *GetName());

		return false;
	}

	if (WaveInfo.TimecodeInfo.IsValid() && 
		(TimecodeInfo != *WaveInfo.TimecodeInfo.Get() || 
			TimecodeInfo.OriginatorDate != WaveInfo.TimecodeInfo->OriginatorDate ||
			TimecodeInfo.Description != WaveInfo.TimecodeInfo->Description ||
			TimecodeInfo.OriginatorReference != WaveInfo.TimecodeInfo->OriginatorReference ||
			TimecodeInfo.OriginatorDescription != WaveInfo.TimecodeInfo->OriginatorDescription || 
			TimecodeInfo.OriginatorTime != WaveInfo.TimecodeInfo->OriginatorTime))
	{
		Modify();
		SetTimecodeInfo(*WaveInfo.TimecodeInfo);
		MarkPackageDirty();

		return true;
	}

	return false;
}

#endif //WITH_EDITORONLY_DATA

int32 USoundWave::GetResourceSizeForFormat(FName Format)
{
	return GetCompressedDataSize(Format);
}


FName USoundWave::GetExporterName()
{
#if WITH_EDITORONLY_DATA
	if( ChannelOffsets.Num() > 0 && ChannelSizes.Num() > 0 )
	{
		return( FName( "SoundSurroundExporterWAV" ) );
	}
#endif // WITH_EDITORONLY_DATA

	return( FName( "SoundExporterWAV" ) );
}


FString USoundWave::GetDesc()
{
	FString Channels;

	if( NumChannels == 0 )
	{
		Channels = TEXT( "Unconverted" );
	}
#if WITH_EDITORONLY_DATA
	else if( ChannelSizes.Num() == 0 )
	{
		Channels = ( NumChannels == 1 ) ? TEXT( "Mono" ) : TEXT( "Stereo" );
	}
#endif // WITH_EDITORONLY_DATA
	else
	{
		Channels = FString::Printf( TEXT( "%d Channels" ), NumChannels );
	}

	return FString::Printf( TEXT( "%3.2fs %s" ), Duration, *Channels );
}

void USoundWave::GetAssetRegistryTags(TArray<FAssetRegistryTag>& OutTags) const
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS;
	Super::GetAssetRegistryTags(OutTags);
	PRAGMA_ENABLE_DEPRECATION_WARNINGS;
}

void USoundWave::GetAssetRegistryTags(FAssetRegistryTagsContext Context) const
{
	Super::GetAssetRegistryTags(Context);

#if WITH_EDITORONLY_DATA
	if (AssetImportData)
	{
		Context.AddTag( FAssetRegistryTag(SourceFileTagName(), AssetImportData->GetSourceData().ToJson(), FAssetRegistryTag::TT_Hidden) );
	}
#endif
}


void USoundWave::SetSampleRate(uint32 InSampleRate, const bool bFromDecoders)
{
	GetWorkingSoundWaveData()->SetSampleRate(IntCastChecked<int32>(InSampleRate));
	SampleRate = InSampleRate;

	if (bFromDecoders)
	{
		// Ensure that we invalidate our cached sample rate if the FProperty sample rate is changed.
		bSampleRateManuallyReset = true;
		bCachedSampleRateFromPlatformSettings = false;
	}
}

void USoundWave::SetNumFrames(uint32 InNumFrames)
{
	GetWorkingSoundWaveData()->SetNumFrames(InNumFrames);	
}

uint32 USoundWave::GetNumFrames() const
{
	return GetWorkingSoundWaveData()->GetNumFrames();
}

void USoundWave::Serialize( FArchive& Ar )
{
	LLM_SCOPE(ELLMTag::AudioSoundWaves);

	DECLARE_SCOPE_CYCLE_COUNTER( TEXT("USoundWave::Serialize"), STAT_SoundWave_Serialize, STATGROUP_LoadTime );

	Super::Serialize( Ar );
	
	// Bit-pack.
	enum
	{
		CookedFlag					= 1 << 0,
		HasOwnerLoadingBehaviorFlag	= 1 << 1,
		
		LoadingBehaviorShift		= 2,
		LoadingBehaviorMask			= 0b00000111,
	};

	// Pack flags into uint32
	uint32 Flags = 0;
	if (Ar.IsSaving())
	{
		if (Ar.IsCooking())
		{
			Flags |= CookedFlag;

#if WITH_EDITOR
			// If we are going to save the owners loading behavior
			const ISoundWaveLoadingBehaviorUtil::FClassData OwnerBehavior = GetOwnerLoadingBehavior(Ar.CookingTarget());
			if (OwnerBehavior.LoadingBehavior != ESoundWaveLoadingBehavior::Uninitialized)
			{
				UE_LOGF(LogAudio, VeryVerbose, "Saving Owners Loading Behavior: %ls, %ls", 
					Audio::ShortEnumString(EnumToString(OwnerBehavior.LoadingBehavior)), *GetFullName());
				
				Flags |= HasOwnerLoadingBehaviorFlag;
				Flags |= (uint32)OwnerBehavior.LoadingBehavior << LoadingBehaviorShift;
			}
#endif //WITH_EDITOR

		}
	}

	Ar << Flags;

	const bool bCooked = Flags & CookedFlag;

	if (FPlatformProperties::RequiresCookedData() && !bCooked && Ar.IsLoading())
	{
		UE_LOGF(LogAudio, Fatal, "This platform requires cooked packages, and audio data was not cooked into %ls.", *GetFullName());
	}

	Ar.UsingCustomVersion(FFrameworkObjectVersion::GUID);
	Ar.UsingCustomVersion(FUE5MainStreamObjectVersion::GUID);

	if (Ar.IsLoading() && (Ar.UEVer() >= VER_UE4_SOUND_COMPRESSION_TYPE_ADDED) && (Ar.CustomVer(FFrameworkObjectVersion::GUID) < FFrameworkObjectVersion::RemoveSoundWaveCompressionName))
	{
		FName DummyCompressionName;
		Ar << DummyCompressionName;
	}

	bool bShouldStreamSound = false;

#if WITH_EDITORONLY_DATA
	FByteBulkData TempOldBulkData;
	bool bBulkDataConverted = false;

	if (bVirtualizeWhenSilent_DEPRECATED)
	{
		bVirtualizeWhenSilent_DEPRECATED = 0;
		VirtualizationMode = EVirtualizationMode::PlayWhenSilent;
	}

	if (Ar.IsLoading())
	{
		// Modify bLoadedFromCookedData state only when loading from archive
		bLoadedFromCookedData = bCooked;

		ModulationSettings.VersionModulators();
	}

	// If we have deprecated properties with data, migrate the data to the new enum
	if (bUseBinkAudio || bSeekableStreaming)
	{
		// Bink audio takes precedence over bSeekableStreaming
		if (bUseBinkAudio)
		{
			SoundAssetCompressionType = ESoundAssetCompressionType::BinkAudio;
		}
		else if (bSeekableStreaming)
		{
			// PCM was CompressionQuality 100
			if (CompressionQuality == 100)
			{
				SoundAssetCompressionType = ESoundAssetCompressionType::PCM;
			}
			else
			{
				SoundAssetCompressionType = ESoundAssetCompressionType::ADPCM;
			}
		}
		bUseBinkAudio = 0;
		bSeekableStreaming = 0;
	}
#endif // WITH_EDITORONLY_DATA

	if (Ar.IsSaving() || Ar.IsCooking())
	{
#if WITH_ENGINE
		// If there is an AutoStreamingThreshold set for the platform we're cooking to,
		// we use it to determine whether this USoundWave should be streaming:
		const ITargetPlatform* CookingTarget = Ar.CookingTarget();
		if (CookingTarget != nullptr)
		{
			bShouldStreamSound = IsStreaming(*CookingTarget->IniPlatformName());
		}
#endif
	}
	else
	{
		bShouldStreamSound = IsStreaming(nullptr);
	}

	bool bSupportsStreaming = false;
	if (Ar.IsLoading() && FPlatformProperties::SupportsAudioStreaming() && !IsRunningDedicatedServer())
	{
		bSupportsStreaming = true;
	}
	else if (Ar.IsCooking() && Ar.CookingTarget()->SupportsFeature(ETargetPlatformFeatures::AudioStreaming))
	{
		bSupportsStreaming = true;
	}

	SerializeCuePoints(Ar, bCooked && Ar.IsLoading());

	// Save cook tags
#if WITH_EDITOR
	if (Ar.IsCooking() && Ar.IsSaving() && Ar.GetCookContext() && Ar.GetCookContext()->GetCookTagList())
	{
		FCookTagList* CookTags = Ar.GetCookContext()->GetCookTagList();
		TArray<FName> CompressedFormats;
		GetWorkingSoundWaveData()->CompressedFormatData.GetContainedFormats(CompressedFormats);
		CookTags->Add(this, "FormatContainerCount", LexToString(CompressedFormats.Num()));
		TStringBuilder<256> FormatsListStr;
		for (int32 FormatIndex = 0; FormatIndex < CompressedFormats.Num(); FormatIndex++)
		{
			FName BaseFormat = GetBaseFormatFromSuffixedFormat(CompressedFormats[FormatIndex]);
			FormatsListStr << BaseFormat;
			if (FormatIndex != CompressedFormats.Num() - 1)
			{
				FormatsListStr << TEXT(',');
			}
		}
		CookTags->Add(this, "FormatContainerFormats", FormatsListStr.ToString());
	}
#endif

	if (bCooked)
	{
#if WITH_EDITOR
		if (Ar.IsLoading())
		{
			// Temporary workaround for allowing editors to load data that was saved for platforms that had streaming disabled. There is nothing int
			// the serialized data that lets us know what is actually stored on disc, so we have to be explicitly told. Ideally, we'd just store something
			// on disc to say how the serialized data is arranged, but doing so would cause a major patch delta.
			static const bool bSoundWaveDataHasStreamingDisabled = FParse::Param(FCommandLine::Get(), TEXT("SoundWaveDataHasStreamingDisabled"));
			bShouldStreamSound = bShouldStreamSound && !bSoundWaveDataHasStreamingDisabled;
		}
#endif

		// Only want to cook/load full data if we don't support streaming
		if (!bShouldStreamSound || !bSupportsStreaming)
		{
			if (Ar.IsCooking())
			{
#if WITH_ENGINE
				TArray<FName> ActualFormatsToSave;
				const ITargetPlatform* CookingTarget = Ar.CookingTarget();
				if (CookingTarget->AllowAudioVisualData())
				{
					// for now we only support one format per wave
					FName Format = CookingTarget->GetWaveFormat(this);
					const FPlatformAudioCookOverrides* CompressionOverrides = FPlatformCompressionUtilities::GetCookOverrides(*CookingTarget->IniPlatformName());

					GetCompressedData(Format, CompressionOverrides); // Get the data from the DDC or build it
					if (CompressionOverrides)
					{
						FString HashedString = *Format.ToString();
						FPlatformAudioCookOverrides::GetHashSuffix(CompressionOverrides, HashedString);
						FName PlatformSpecificFormat = *HashedString;
						ActualFormatsToSave.Add(PlatformSpecificFormat);
					}
					else
					{
						ActualFormatsToSave.Add(Format);
					}
				}
				bool bMapped = CookingTarget->SupportsFeature(ETargetPlatformFeatures::MemoryMappedFiles) && CookingTarget->SupportsFeature(ETargetPlatformFeatures::MemoryMappedAudio);

				GetWorkingSoundWaveData()->CompressedFormatData.Serialize(Ar, this, &ActualFormatsToSave, true, DEFAULT_ALIGNMENT,
					!bMapped, // inline if not mapped
					bMapped);
#endif
			}
			else
			{
				if (FPlatformProperties::SupportsMemoryMappedFiles() && FPlatformProperties::SupportsMemoryMappedAudio())
				{
					GetWorkingSoundWaveData()->CompressedFormatData.SerializeAttemptMappedLoad(Ar, this);
				}
				else
				{
					GetWorkingSoundWaveData()->CompressedFormatData.Serialize(Ar, this);
				}
			}
		}
	}
	else if (!Ar.IsObjectReferenceCollector()) // Skip bulk data serialization when collecting references
	{
		// only save the raw data for non-cooked packages
#if WITH_EDITORONLY_DATA
		FScopeLock ScopeLock(&RawDataCriticalSection);

		if (Ar.IsLoading() && Ar.CustomVer(FUE5MainStreamObjectVersion::GUID) < FUE5MainStreamObjectVersion::SoundWaveVirtualizationUpdate)
		{
			TempOldBulkData.Serialize(Ar, this, INDEX_NONE, false);	
			bBulkDataConverted = true;
		}
		else
		{
			RawData.Serialize(Ar, this);
		}
#endif
	}

	Ar << CompressedDataGuid;

	// Use our owners loading behavior if we've been cooked to do so.
	if (Ar.IsLoading() && (Flags & CookedFlag) && (Flags & HasOwnerLoadingBehaviorFlag) && LoadingBehavior != ESoundWaveLoadingBehavior::ForceInline)
	{
		const ESoundWaveLoadingBehavior OwnerBehavior = (ESoundWaveLoadingBehavior)((Flags >> LoadingBehaviorShift) & LoadingBehaviorMask);
		LoadingBehavior = OwnerBehavior;
	}

#if WITH_EDITORONLY_DATA	
	// If we're converting to FEditorBulkData, we need to first serialize the CompressedDataGuid, as we use it as our key.
	if (bBulkDataConverted)
	{
		FScopeLock ScopeLock(&RawDataCriticalSection);
		RawData.CreateFromBulkData(TempOldBulkData, CompressedDataGuid, this); 
	}
#endif //WITH_EDITORONLY_DATA	
	
	if (bShouldStreamSound)
	{
		if (bCooked)
		{
			// only cook/load streaming data if it's supported
			if (bSupportsStreaming)
			{
				SerializeCookedPlatformData(Ar);
			}
		}
	}

	if (!(IsTemplate() || IsRunningDedicatedServer()) && Ar.IsLoading())
	{
		// For non-editor builds, we can immediately cache the sample rate.
		SampleRate = GetSampleRateForCurrentPlatform();
	}
}

ESoundAssetCompressionType USoundWave::GetSoundAssetCompressionType() const
{
	// If we are told to use the proejct-defined asset compression type, then we need to look it up
	if (SoundAssetCompressionType == ESoundAssetCompressionType::ProjectDefined)
	{
#if WITH_EDITOR
		// In the editor, always retrieve the settings
		UAudioSettings* Settings = GetMutableDefault<UAudioSettings>();
		return Audio::ToSoundAssetCompressionType(Settings->DefaultAudioCompressionType);
#else
		// Only query the project settings once when retrieving the asset compression type when not running with the editor
		static bool bDefaultCompressionTypeCached = false;
		static ESoundAssetCompressionType DefaultAudioCompressionType = ESoundAssetCompressionType::PlatformSpecific;

		if (!bDefaultCompressionTypeCached)
		{
			const UAudioSettings* Settings = GetDefault<UAudioSettings>();
			DefaultAudioCompressionType = Audio::ToSoundAssetCompressionType(Settings->DefaultAudioCompressionType);
			bDefaultCompressionTypeCached = true;
		}
		return DefaultAudioCompressionType;
#endif
	}
	else
	{
		return SoundAssetCompressionType;
	}
}

ESoundAssetCompressionType USoundWave::GetSoundAssetCompressionTypeEnum() const
{
	return SoundAssetCompressionType;
}

void USoundWave::SetSoundAssetCompressionType(ESoundAssetCompressionType InSoundAssetCompressionType, bool bMarkDirty)
{
#if WITH_EDITOR

	
	/*
	 * This is a bit of a hack to make viable for a hot-fixable.
	 * If bMarkDirty is False and the current compression type is project defined,
	 * it means the project default codec was changed.  In this scenario we don't want
	 * to change the SoundAssetCompressionType field, as this will change the property
	 * on the USoundWave to whatever the new project default is.
	 */
	if(bMarkDirty || SoundAssetCompressionType != ESoundAssetCompressionType::ProjectDefined)
	{
		SoundAssetCompressionType = InSoundAssetCompressionType;
	}

	FName RuntimeFormat = GetWorkingSoundWaveData()->FindRuntimeFormat(*this);
	if (RuntimeFormat.IsNone())
	{
		RuntimeFormat = Audio::ToName(SoundAssetCompressionType);
	}

	GetWorkingSoundWaveData()->CacheRuntimeFormatDependentState(RuntimeFormat);

	UpdateAsset(bMarkDirty);
#endif // #if WITH_EDITOR
}

TArray<FSoundWaveCuePoint> USoundWave::GetCuePoints() const
{
	return GetWorkingSoundWaveData()->GetCuePoints();
}

void USoundWave::SetAllCuePoints(TArray<FSoundWaveCuePoint> InCuePoints) const
{
	GetWorkingSoundWaveData()->SetAllCuePoints(InCuePoints);
}

#if WITH_EDITOR
TArray<FSoundWaveCuePoint> USoundWave::GetSoundWaveCuePoints() const
{
	return CuePoints;
}

void USoundWave::SetSoundWaveCuePoints(const TArray<FSoundWaveCuePoint>& InCuePoints)
{
	CuePoints = InCuePoints;
	SetAllCuePoints(InCuePoints);
}

const ESoundWaveCuePointOrigin USoundWave::GetCuePointOrigin() const
{
	return CuePointOrigin;
}

void USoundWave::SetCuePointOrigin(const ESoundWaveCuePointOrigin InCuePointOrigin)
{
	CuePointOrigin = InCuePointOrigin;
}

void USoundWave::SetLoudnessValues(const float InLUFS, const float InSamplePeakDB)
{
	LUFS = InLUFS;
	SamplePeakDB = InSamplePeakDB;
}
#endif //WITH_EDITOR

TArray<FSoundWaveCuePoint> USoundWave::GetLoopRegions() const
{
	return GetWorkingSoundWaveData()->GetLoopRegions();
}

FName USoundWave::GetRuntimeFormat() const
{
	return GetWorkingSoundWaveData()->GetRuntimeFormat();
}

void USoundWave::ResetInvalidTemplateArguments()
{
	NumChannels = 0;

#if WITH_EDITOR
	TotalSamples = 0;
	ChannelOffsets.Reset();
	ChannelSizes.Reset();
	Transformations.Empty();
#endif //WITH_EDITOR
}

const TSharedRef<FSoundWaveData>& USoundWave::GetWorkingSoundWaveData() const
{
#if UE_USOUNDWAVE_HOT_RELOADING_ENABLED
	return  WorkingSoundWaveData;
#else // #if UE_USOUNDWAVE_HOT_RELOADING_ENABLED
	return Proxy->PublishedSoundWaveData;
#endif // #if UE_USOUNDWAVE_HOT_RELOADING_ENABLED
}

float USoundWave::GetSubtitlePriority() const
{
	return SubtitlePriority;
};

bool USoundWave::SupportsSubtitles() const
{
	return VirtualizationMode == EVirtualizationMode::PlayWhenSilent || (Subtitles.Num() > 0);
}

void USoundWave::PostInitProperties()
{
	Super::PostInitProperties();

	if(!IsTemplate())
	{
		// Don't rebuild our streaming chunks yet because we may not have loaded the RawPCMData at this point.
		InvalidateCompressedData(false, false);
	}

	MaxDistance = ComputeMaxDistance();

#if WITH_EDITORONLY_DATA
	if (!HasAnyFlags(RF_ClassDefaultObject))
	{
		AssetImportData = NewObject<UAssetImportData>(this, TEXT("AssetImportData"));
	}
#endif
}

bool USoundWave::HasCompressedData(FName Format, ITargetPlatform* TargetPlatform) const
{
	if (IsTemplate() || IsRunningDedicatedServer())
	{
		return false;
	}

	const FPlatformAudioCookOverrides* CompressionOverrides = nullptr;

	if (GIsEditor)
	{
		if (TargetPlatform)
		{
			CompressionOverrides = FPlatformCompressionUtilities::GetCookOverrides(*TargetPlatform->IniPlatformName());
		}
	}
	else
	{
		// TargetPlatform is not available on consoles/mobile, so we have to grab it ourselves:
		CompressionOverrides = FPlatformCompressionUtilities::GetCookOverrides();
	}


	if (CompressionOverrides)
	{
#if WITH_EDITOR
		FName PlatformSpecificFormat;
		FString HashedString = *Format.ToString();
		FPlatformAudioCookOverrides::GetHashSuffix(CompressionOverrides, HashedString);
		PlatformSpecificFormat = *HashedString;
#else
		// on non-editor builds, we cache the concatenated format in a static FName.
		static FName PlatformSpecificFormat;
		static FName CachedFormat;
		if (!Format.IsEqual(CachedFormat))
		{
			FString HashedString = *Format.ToString();
			FPlatformAudioCookOverrides::GetHashSuffix(CompressionOverrides, HashedString);
			PlatformSpecificFormat = *HashedString;

			CachedFormat = Format;
		}
#endif // WITH_EDITOR
		return GetWorkingSoundWaveData()->CompressedFormatData.Contains(PlatformSpecificFormat);
	}
	else
	{
		return GetWorkingSoundWaveData()->CompressedFormatData.Contains(Format);
	}

}

const FPlatformAudioCookOverrides* USoundWave::GetPlatformCompressionOverridesForCurrentPlatform()
{
	return FPlatformCompressionUtilities::GetCookOverrides();
}

#if WITH_EDITOR
bool USoundWave::GetImportedSoundWaveData(TArray<uint8>& OutRawPCMData, uint32& OutSampleRate, uint16& OutNumChannels) const
{
	TArray<EAudioSpeakers> ChannelOrder;

	bool bResult = GetImportedSoundWaveData(OutRawPCMData, OutSampleRate, ChannelOrder);
	
	if (bResult)
	{
		OutNumChannels = ChannelOrder.Num();
	}
	else
	{
		OutNumChannels = 0;
	}

	return bResult;
}

bool USoundWave::GetImportedSoundWaveData(TArray<uint8>& OutRawPCMData, uint32& OutSampleRate, TArray<EAudioSpeakers>& OutChannelOrder) const
{
	OutRawPCMData.Reset();
	OutSampleRate = 0;
	OutChannelOrder.Reset();

#if WITH_EDITORONLY_DATA
	FScopeLock ScopeLock(&RawDataCriticalSection);
#endif

	// Can only get sound wave data if there is bulk data 
	if (RawData.HasPayloadData())
	{
		TFuture<FSharedBuffer> BufferFuture = RawData.GetPayload();
		const uint8* Data = (const uint8*) BufferFuture.Get().GetData(); // Will block.
		int32 DataSize = BufferFuture.Get().GetSize();

		if (NumChannels > 2)
		{
			static const EAudioSpeakers DefaultChannelOrder[SPEAKER_Count] = 
			{
				SPEAKER_FrontLeft,
				SPEAKER_FrontRight,
				SPEAKER_FrontCenter,
				SPEAKER_LowFrequency,
				SPEAKER_LeftSurround,
				SPEAKER_RightSurround,
				SPEAKER_LeftBack,
				SPEAKER_RightBack
			};

			TFunction<bool(const FWaveModInfo& WaveModInfo)> ValidateWaveModInfo = [](const FWaveModInfo& WaveModInfo)
			{
				// Check for valid channel count
				if (1 != *WaveModInfo.pChannels)
				{
					UE_LOGF(LogAudio, Warning, "Cannot handle audio format. Expected single channel audio but read %d channels", *WaveModInfo.pChannels);
					
					return false;
				}

				// Check for valid bit depth
				if (16 != *WaveModInfo.pBitsPerSample)
				{
					UE_LOGF(LogAudio, Warning, "Cannot handle audio format. Expected 16bit audio but found %d bit audio", *WaveModInfo.pBitsPerSample);
					
					return false;
				}

				return true;
			};

			check(ChannelOffsets.Num() == ChannelSizes.Num());
			check(ChannelOffsets.Num() == SPEAKER_Count);

			// Multichannel audio with more than 2 channels must be accessed by
			// inspecting the channel offsets and channel sizes of the USoundWave.

			// Determine which channel is the smallest and read the wave for the number of frames so we can truncate larger channels if needed
			int32 SmallestValidChannelIndex = INDEX_NONE;
			int32 SmallestChannelSize = INT32_MAX;

			for (int32 Index = 0; Index < ChannelOffsets.Num(); Index++)
			{
				if (ChannelSizes[Index] > 0 && ChannelSizes[Index] < SmallestChannelSize)
				{
					SmallestValidChannelIndex = Index;
					SmallestChannelSize = ChannelSizes[Index];
				}
			}
			
			if (SmallestValidChannelIndex == INDEX_NONE)
			{
				UE_LOGF(LogAudio, Warning, "Failed to read wave data: %ls.", *GetFullName());
				
				return false;
			}

			FWaveModInfo SmallestChannelWaveInfo;
			if (!SmallestChannelWaveInfo.ReadWaveHeader(Data, ChannelSizes[SmallestValidChannelIndex], ChannelOffsets[SmallestValidChannelIndex]))
			{
				UE_LOGF(LogAudio, Warning, "Failed to read wave data: %ls.", *GetFullName());
				
				return false;
			}

			if (!ValidateWaveModInfo(SmallestChannelWaveInfo))
			{
				return false;
			}

			check(SmallestChannelWaveInfo.SampleDataSize)

			// The largest number of frames a channel should have in this soundwave
			// Bad chunk data can lead to NumFramesPerChannel being a decimal value but the data truncation to int32 removes the bad sample.
			int32 NumFramesPerChannel = SmallestChannelWaveInfo.SampleDataSize / sizeof(int16);
			int32 NumSamples = NumFramesPerChannel * NumChannels; // Number of total samples across all channels
			OutSampleRate = *SmallestChannelWaveInfo.pSamplesPerSec;

			if (NumSamples > 0)
			{
				// Translate NumSamples to bytes because OutRawPCMData is in bytes. 
				const int32 NumBytes = NumSamples * sizeof(int16);
				OutRawPCMData.AddZeroed(NumBytes);
			}
			else
			{
				UE_LOGF(LogAudio, Warning, "Zero samples found in wave data: %ls.", *GetFullName());

				return false;
			}

			// Determine which channels have data and Interleave channel data
			for (int32 ChannelIndex = 0; ChannelIndex < ChannelOffsets.Num(); ChannelIndex++)
			{
				if (ChannelSizes[ChannelIndex] <= 0)
				{
					continue;
				}

				FWaveModInfo WaveInfo;

				// parse the wave data for a single channel
				if (!WaveInfo.ReadWaveHeader(Data, ChannelSizes[ChannelIndex], ChannelOffsets[ChannelIndex]))
				{
					UE_LOGF(LogAudio, Warning, "Failed to read wave data: %ls.", *GetFullName());
					return false;
				}

				if (!ValidateWaveModInfo(WaveInfo))
				{
					return false;
				}

				// Set output channel type
				OutChannelOrder.Add(DefaultChannelOrder[ChannelIndex]);

				check(OutSampleRate == *WaveInfo.pSamplesPerSec);
				const int32 ThisChannelNumFrames = WaveInfo.SampleDataSize / sizeof(int16);

				TArrayView<const int16> ChannelArrayView;
				// If not all channels have the same number of frames we will have to copy the channel data and alter it
				TArray<int16> ChannelArrayCopy;

				if (ThisChannelNumFrames == NumFramesPerChannel)
				{
					ChannelArrayView = TArrayView<const int16>(reinterpret_cast<const int16*>(WaveInfo.SampleDataStart), NumFramesPerChannel);
				}
				else // Adjust if not all channels have the same number of frames
				{
					UE_LOGF(LogAudio, Warning, "Audio channels contain varying number of frames (%d vs %d). Truncating frames.", NumFramesPerChannel, ThisChannelNumFrames);
					check(NumFramesPerChannel < ThisChannelNumFrames);

					// Copy the data instead of referencing it directly so we don't modify the imported WaveInfo
					ChannelArrayCopy.Append(reinterpret_cast<const int16*>(WaveInfo.SampleDataStart), ThisChannelNumFrames);

					constexpr EAllowShrinking AllowShrinking = EAllowShrinking::Yes;
					ChannelArrayCopy.SetNum(NumFramesPerChannel, AllowShrinking); //Truncate 

					ChannelArrayView = TArrayView<const int16>(ChannelArrayCopy);
				}

				// Interleave each channel's data into one multichannel audio buffer
				TArrayView<int16> OutPCM(reinterpret_cast<int16*>(OutRawPCMData.GetData()), NumSamples);

				check(OutChannelOrder.Num() >= 1);
				int32 DestSamplePos = OutChannelOrder.Num() - 1;
				int32 SourceSamplePos = 0;

				while (DestSamplePos < NumSamples)
				{
					OutPCM[DestSamplePos] = ChannelArrayView[SourceSamplePos];

					SourceSamplePos++;
					DestSamplePos += NumChannels;
				}
			}
		}
		else
		{
			FWaveModInfo WaveInfo;

			// parse the wave data
			if (!WaveInfo.ReadWaveHeader(Data, DataSize, 0))
			{
				UE_LOGF(LogAudio, Warning, "Only mono or stereo 16 bit waves allowed: %ls.", *GetFullName());
				return false;
			}

			check(NumChannels > 0);
			const bool InvalidDataSize = (WaveInfo.SampleDataSize % sizeof(int16) != 0);
			uint32 NumSamples = WaveInfo.SampleDataSize / sizeof(int16);

			if (InvalidDataSize || NumSamples % NumChannels != 0)
			{
				UE_LOGF(LogAudio, Warning, "Invalid Sample Data Size compared to NumChannels for: %ls. Adjusting sample count", *GetFullName());

				NumSamples -= NumSamples % NumChannels;
			}

			// NumSamples * sizeof(int16) can be different from WaveInfo.DataSize if the file contains bad data
			const uint32 SampleDataSize = NumSamples * sizeof(int16);

			// Copy the raw PCM data and the header info that was parsed
			OutRawPCMData.AddUninitialized(SampleDataSize);
			FMemory::Memcpy(OutRawPCMData.GetData(), WaveInfo.SampleDataStart, SampleDataSize);

			OutSampleRate = *WaveInfo.pSamplesPerSec;
			
			if (1 == *WaveInfo.pChannels)
			{
				OutChannelOrder.Add(SPEAKER_FrontLeft);
			}
			else if (2 == *WaveInfo.pChannels)
			{
				OutChannelOrder.Append({SPEAKER_FrontLeft, SPEAKER_FrontRight});
			}
		}

		return true;
	}

	UE_LOGF(LogAudio, Warning, "Failed to get imported raw data for sound wave '%ls'", *GetFullName());
	return false;
}
#endif

FName USoundWave::GetPlatformSpecificFormat(FName Format, const FPlatformAudioCookOverrides* CompressionOverrides)
{
	// Platforms that require compression overrides get concatenated formats.
#if WITH_EDITOR
	FName PlatformSpecificFormat;
	if (CompressionOverrides)
	{
		FString HashedString = *Format.ToString();
		FPlatformAudioCookOverrides::GetHashSuffix(CompressionOverrides, HashedString);
		PlatformSpecificFormat = *HashedString;
	}
	else
	{
		PlatformSpecificFormat = Format;
	}
#else
	if (CompressionOverrides == nullptr)
	{
		CompressionOverrides = GetPlatformCompressionOverridesForCurrentPlatform();
	}

	// Cache the concatenated hash:
	static FName PlatformSpecificFormat;
	static FName CachedFormat;
	if (!Format.IsEqual(CachedFormat))
	{
		if (CompressionOverrides)
		{
			FString HashedString = *Format.ToString();
			FPlatformAudioCookOverrides::GetHashSuffix(CompressionOverrides, HashedString);
			PlatformSpecificFormat = *HashedString;
		}
		else
		{
			PlatformSpecificFormat = Format;
		}

		CachedFormat = Format;
	}

#endif // WITH_EDITOR

	return PlatformSpecificFormat;
}

void USoundWave::BeginGetCompressedData(FName Format, const FPlatformAudioCookOverrides* CompressionOverrides, const ITargetPlatform* InTargetPlatform)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(USoundWave::BeginGetCompressedData);

#if WITH_EDITOR
	if (IsTemplate() || IsRunningDedicatedServer() || bLoadedFromCookedData)
	{
		return;
	}

	// If stream caching has been enabled or disabled since the previous DDC operation, we need to invalidate the current
	InvalidateSoundWaveIfNeccessary();

	FName PlatformSpecificFormat = GetPlatformSpecificFormat(Format, CompressionOverrides);

	if (!GetWorkingSoundWaveData()->CompressedFormatData.Contains(PlatformSpecificFormat) && !AsyncLoadingDataFormats.Contains(PlatformSpecificFormat))
	{
		if (GetDerivedDataCache())
		{
			COOK_STAT(auto Timer = SoundWaveCookStats::UsageStats.TimeSyncWork());
			COOK_STAT(Timer.TrackCyclesOnly());
			FDerivedAudioDataCompressor* DeriveAudioData = new FDerivedAudioDataCompressor(this, Format, PlatformSpecificFormat, CompressionOverrides, InTargetPlatform);
			uint32 GetHandle = GetDerivedDataCacheRef().GetAsynchronous(DeriveAudioData);
			AsyncLoadingDataFormats.Add(PlatformSpecificFormat, GetHandle);
		}
		else
		{
			UE_LOGF(LogAudio, Error, "Attempt to access the DDC when there is none available on sound '%ls', format = %ls.", *GetFullName(), *PlatformSpecificFormat.ToString());
		}
	}
#else
	// No async DDC read in non-editor, nothing to precache
#endif
}

#if WITH_EDITOR

bool USoundWave::IsCompressedDataReady(FName Format, const FPlatformAudioCookOverrides* CompressionOverrides) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(USoundWave::IsCompressedDataReady);

	FName PlatformSpecificFormat = GetPlatformSpecificFormat(Format, CompressionOverrides);

	const uint32* AsyncHandle = AsyncLoadingDataFormats.Find(PlatformSpecificFormat);
	if (AsyncHandle)
	{
		return GetDerivedDataCacheRef().PollAsynchronousCompletion(*AsyncHandle);
	}
	
	return true;
}

bool USoundWave::IsLoadedFromCookedData() const
{
	return bLoadedFromCookedData;
}

#endif

FByteBulkData* USoundWave::GetCompressedData(FName Format, const FPlatformAudioCookOverrides* CompressionOverrides, const ITargetPlatform* InTargetPlatform)
{ 
	if (IsTemplate() || IsRunningDedicatedServer())
	{
		return nullptr;
	}

	TRACE_CPUPROFILER_EVENT_SCOPE(USoundWave::GetCompressedData);
	FName PlatformSpecificFormat = GetPlatformSpecificFormat(Format, CompressionOverrides);

	bool bContainedValidData = GetWorkingSoundWaveData()->CompressedFormatData.Contains(PlatformSpecificFormat);
	FByteBulkData* Result = &GetWorkingSoundWaveData()->CompressedFormatData.GetFormat(PlatformSpecificFormat);
	if (!bContainedValidData)
	{
		if (!FPlatformProperties::RequiresCookedData() && GetDerivedDataCache())
		{
			TArray<uint8> OutData;
			bool bDataWasBuilt = false;
			bool bGetSuccessful = false;

			uint32* AsyncHandle = nullptr;
#if WITH_EDITOR
			AsyncHandle = AsyncLoadingDataFormats.Find(PlatformSpecificFormat);
#endif

			COOK_STAT(auto Timer = AsyncHandle ? SoundWaveCookStats::UsageStats.TimeAsyncWait() : SoundWaveCookStats::UsageStats.TimeSyncWork());

#if WITH_EDITOR
			if (AsyncHandle)
			{
				GetDerivedDataCacheRef().WaitAsynchronousCompletion(*AsyncHandle);
				bGetSuccessful = GetDerivedDataCacheRef().GetAsynchronousResults(*AsyncHandle, OutData, &bDataWasBuilt);
				AsyncLoadingDataFormats.Remove(PlatformSpecificFormat);
			}
			else
#endif
			{
				FDerivedAudioDataCompressor* DeriveAudioData = new FDerivedAudioDataCompressor(this, Format, PlatformSpecificFormat, CompressionOverrides, InTargetPlatform);
				bGetSuccessful = GetDerivedDataCacheRef().GetSynchronous(DeriveAudioData, OutData, &bDataWasBuilt);
			}

			if (bGetSuccessful)
			{
				COOK_STAT(Timer.AddHitOrMiss(bDataWasBuilt ? FCookStats::CallStats::EHitOrMiss::Miss : FCookStats::CallStats::EHitOrMiss::Hit, OutData.Num()));

				FAudioCookOutputs CookOutput;
				FMemoryReader Reader(OutData);
				ensure(CookOutput.Serialize(Reader));
				
				Result->Lock(LOCK_READ_WRITE);
				FMemory::Memcpy(Result->Realloc(CookOutput.EncodedData.Num()), CookOutput.EncodedData.GetData(), CookOutput.EncodedData.Num());
				Result->Unlock();
			}
		}
		else
		{
			UE_LOGF(LogAudio, Error, "Attempt to access the DDC when there is none available on sound '%ls', format = %ls. Should have been cooked.", *GetFullName(), *PlatformSpecificFormat.ToString());
		}
	}
	else
	{
		// We had valid data in the format container - it should be non zero size!
		// If we have a DDC then we don't want to log because we emplaced an empty buffer on purpose to
		// avoid hitting the ddc over and over.
#if !WITH_EDITOR
		if (Result->GetBulkDataSize() == 0)
		{
			UE_LOGF(LogAudio, Warning, "Sound format container returned empty sound! sound '%ls', format = %ls.", *GetFullName(), *PlatformSpecificFormat.ToString());
		}
#endif
	}
	check(Result);
	return Result->GetBulkDataSize() > 0 ? Result : NULL; // we don't return empty bulk data...but we save it to avoid thrashing the DDC
}

void USoundWave::InvalidateCompressedData(bool bFreeResources, bool bRebuildStreamingChunks)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(USoundWave::InvalidateCompressedData);

	CompressedDataGuid = FGuid::NewGuid();
	
	// make a new copy of working data so we don't affect the published object
#if UE_USOUNDWAVE_HOT_RELOADING_ENABLED
	WorkingSoundWaveData = MakeShared<FSoundWaveData>(*this);
	Proxy->Publish(WorkingSoundWaveData);
#endif // #if UE_USOUNDWAVE_HOT_RELOADING_ENABLED

#if WITH_EDITOR
	// In editor, do not modify SoundWaveDataPtr because it may be accessed by 
	// another system through the FSoundWaveProxy. Instead, create a new SoundWaveDataPtr
	// to ensure that SoundWaveDataPtr is not being read while it is being written to.
	const bool bIsRetained = GetWorkingSoundWaveData()->FirstChunk.IsValid();
#else
	GetWorkingSoundWaveData()->DiscardZerothChunkData();
	GetWorkingSoundWaveData()->CompressedFormatData.FlushData();
	RemoveAudioResource();
#endif // WITH_EDITOR

	if (bFreeResources)
	{
		FreeResources(false);
	}

#if WITH_EDITOR
	if (bFreeResources)
	{
		// Flush any in-flight async loading data formats to make sure their results
		// are not during the next call to GetCompressedData
		FlushAsyncLoadingDataFormats();
	}

	if (bRebuildStreamingChunks)
	{
		CachePlatformData(true /* bAsyncCache */);

		const bool bHasSampleData = !(bIsSourceBus || bProcedural);
		if (bHasSampleData)
		{
			// Source buses and procedural sources do not have sample data and should 
			// avoid going through creation of streaming compressed data.
			//
			// procedural sources and source buses should be rebased to a yet-to-be
			// created `USoundWaveBase`
			GetWorkingSoundWaveData()->InitializeDataFromSoundWave(*this);
		}

		CurrentChunkRevision += 1;
		GetWorkingSoundWaveData()->CurrentChunkRevision = CurrentChunkRevision;
	}

	// If this sound wave is retained, release and re-retain the new chunk.
	if (bIsRetained)
	{
		RetainCompressedAudio(true);
	}
#endif
}

#if WITH_EDITOR
void USoundWave::FlushAsyncLoadingDataFormats()
{
	// Flush any async results so we don't leak them in the DDC
	if (GetDerivedDataCache() && AsyncLoadingDataFormats.Num() > 0)
	{
		TArray<uint8> OutData;
		for (auto AsyncLoadIt = AsyncLoadingDataFormats.CreateConstIterator(); AsyncLoadIt; ++AsyncLoadIt)
		{
			uint32 AsyncHandle = AsyncLoadIt.Value();
			GetDerivedDataCacheRef().WaitAsynchronousCompletion(AsyncHandle);
			GetDerivedDataCacheRef().GetAsynchronousResults(AsyncHandle, OutData);
		}

		AsyncLoadingDataFormats.Empty();
	}
}

void USoundWave::FlushAudioRenderingCommands() const
{
	check(IsInAudioThread());

	// Waits for audio rendering commands to execute
	if (GEngine && !GExitPurge)
	{
		FAudioDeviceManager* DeviceManager = GEngine->GetAudioDeviceManager();
		if (DeviceManager)
		{
			DeviceManager->IterateOverAllDevices([](Audio::FDeviceId InDeviceID, FAudioDevice* InDevice)
			{
				if (InDevice)
				{
					InDevice->FlushAudioRenderingCommands(false /* bPumpSynchronously */);
				}
			});
		}
	}
}
#endif

bool USoundWave::HasStreamingChunks() const
{
	// Only checking to see if it is set to ForceInline. ForceInline is not supported on 
	// USoundClasses, so it is safe to ignore USoundClasses when calling `GetLoadingBehavior(...)`
	const bool bIsForceInline = (GetLoadingBehavior(false /* bCheckSoundClasses */) == ESoundWaveLoadingBehavior::ForceInline);
	const bool bHasChunks = (GetWorkingSoundWaveData()->RunningPlatformData.GetNumChunks() > 0);
	return !bIsForceInline && bHasChunks;
}

void USoundWave::PostLoad()
{
	LLM_SCOPE(ELLMTag::AudioSoundWaves);

	Super::PostLoad();

	if (GetOutermost()->HasAnyPackageFlags(PKG_ReloadingForCooker))
	{
		return;
	}

#if WITH_EDITORONLY_DATA
	// Log a warning after loading if the source has effect chains but has channels greater than 2.
	if (SourceEffectChain && SourceEffectChain->Chain.Num() > 0 && NumChannels > 2)
	{
		UE_LOGF(LogAudio, Warning, "Sound Wave '%ls' has defined an effect chain but is not mono or stereo.", *GetName());
	}


	if (RawData.HasPayloadData())
	{
		if (ImportedSampleRate == 0)
		{
			// update ImportedSampleRate ASAP so that it reflects the 
			// will only be valid if this sound wave was from an imported .wav file. 
			// otherwise we can safely ignore it
			TArray<uint8> ImportedRawPCMData;
			uint32 OriginalSampleRate = 0;
			uint16 ImportedNumChannels = 0;
			if (GetImportedSoundWaveData(ImportedRawPCMData, OriginalSampleRate, ImportedNumChannels))
			{
				ImportedSampleRate = OriginalSampleRate;
			}
		}
	}
	
	if (TotalSamples == 0 && ImportedSampleRate != 0)
	{
		TotalSamples = Duration * ImportedSampleRate;
	}
#endif

	MaxDistance = ComputeMaxDistance();

	// Don't need to do anything in post load if this is a source bus or procedural audio
	if (this->IsA(USoundSourceBus::StaticClass()) || bProcedural)
	{
		return;
	}

	CacheInheritedLoadingBehavior();
	
	if (FApp::CanEverRenderAudio())
	{
#if WITH_EDITORONLY_DATA
		if (IsStreaming(nullptr) && FPlatformProperties::SupportsAudioStreaming())
		{
			CachePlatformData(true /* bAsyncCache */);
		}
#endif // #if WITH_EDITORONLY_DATA

		// update shared state
		if (!HasAnyFlags(RF_ClassDefaultObject))
		{
			GetWorkingSoundWaveData()->InitializeDataFromSoundWave(*this);
		}
		
		ESoundWaveLoadingBehavior ActualLoadingBehavior = GetLoadingBehavior();
		Proxy->InitializeFromWave(*this);

#if UE_USOUNDWAVE_HOT_RELOADING_ENABLED
		Proxy->Publish(WorkingSoundWaveData);
#endif // #if UE_USOUNDWAVE_HOT_RELOADING_ENABLED

		if (ShouldUseStreamCaching())
		{
#if !WITH_EDITOR
			const TIndirectArray<FStreamedAudioChunk>& Chunks = GetWorkingSoundWaveData()->RunningPlatformData.GetChunks();

			// First chunk can be optionally inlined for the sake of pre-buffering.
			const bool bIsFirstChunkInlined = Chunks.IsValidIndex(1) ? Chunks[1].BulkData.IsInlined() : false;
#else
			static constexpr bool bIsFirstChunkInlined = false; 
#endif //WITH_EDITOR			

			// Allowing retaining if we're NOT in the editor AND we've NOT disabled retaining (via a cvar).
			const bool bAllowRetaining = !GIsEditor && !DisableRetainingCVar;
			
			// Retaining the "inlined" first chunk (which can be any load mode) is enabled via a CVAR.
			const bool bShouldRetainInlinedFirstChunked = bIsFirstChunkInlined && !!bRetainInlinedFirstChunkCVar;

			// We retain the first chunk if we're set to retain
			// OR we've inlined the first chunk (we do when we're pre-buffering). We don't want these dumped by the stream cache if we can help it.
			const bool bShouldRetainFirstChunk = bAllowRetaining &&
				(ActualLoadingBehavior == ESoundWaveLoadingBehavior::RetainOnLoad || bShouldRetainInlinedFirstChunked);

			UE_CLOGF(Audio::MatchesLogFilter(*GetName()), LogAudioTiming, Verbose, "PostLoad: Wave='%ls'\tFormat='%ls'\tLooping=%ls\tChunk1(Retain=%ls,Inlined=%ls),LB=%ls", 
				*GetFName().ToString(),
				*GetRuntimeFormat().ToString(),
				ToCStr(LexToString(bLooping)),
				ToCStr(LexToString(bShouldRetainFirstChunk)),
				ToCStr(LexToString(bIsFirstChunkInlined)),
				Audio::ShortEnumString(EnumToString(ActualLoadingBehavior))
			);
						
			if (bShouldRetainFirstChunk)
			{
				RetainCompressedAudio(false);
			}
			else
			{
				// if a sound class defined our loading behavior as something other than Retain and our cvar default is to retain, we need to release our handle.
				ReleaseCompressedAudio();

				bool bShouldPrime = (ActualLoadingBehavior == ESoundWaveLoadingBehavior::PrimeOnLoad);
				bShouldPrime |= (GIsEditor && (ActualLoadingBehavior == ESoundWaveLoadingBehavior::RetainOnLoad)); // treat this scenario like PrimeOnLoad

				if (bShouldPrime) 
				{
					const bool bHasMultipleChunks = GetNumChunks() > 1;
					if (bHasMultipleChunks)
					{
						IStreamingManager::Get().GetAudioStreamingManager().RequestChunk(GetWorkingSoundWaveData(), 1, [](EAudioChunkLoadResult) {});
					}
				}
			}

			// If DisableRetainingCVar was set after this USoundWave was loaded by the ALT, release its compressed audio here.
			if (DisableRetainingCVar)
			{
				ReleaseCompressedAudio();
			}

			if (!GIsEditor)
			{
				// In case any code accesses bStreaming directly, we fix up bStreaming based on the current platform's cook overrides.
				bStreaming = IsStreaming(nullptr);
			}
		}
	}

	// Compress to whatever formats the active target platforms want
	// static here as an optimization
	ITargetPlatformManagerModule* TPM = GetTargetPlatformManager();
#if WITH_EDITORONLY_DATA
	const bool bShouldLoadCompressedData = !(bLoadedFromCookedData && IsRunningCommandlet());
#else
	const bool bShouldLoadCompressedData = true;
#endif
	// Don't anticipate use of compressed data for streamed sound. The CachePlatformData call will request it on demand instead.
	if (TPM && bShouldLoadCompressedData && !IsStreaming(nullptr))
	{
		const TArray<ITargetPlatform*>& Platforms = TPM->GetActiveTargetPlatforms();

		for (const ITargetPlatform* Platform : Platforms)
		{
			if (Platform->AllowAudioVisualData())
			{
				BeginGetCompressedData(Platform->GetWaveFormat(this), FPlatformCompressionUtilities::GetCookOverrides(*Platform->IniPlatformName()), Platform);
			}
		}
	}

	
	// We don't precache default objects and we don't precache in the Editor as the latter will
	// most likely cause us to run out of memory.
	if (!GIsEditor && !IsTemplate(RF_ClassDefaultObject) && GEngine)
	{
		FAudioDevice* AudioDevice = GEngine->GetMainAudioDeviceRaw();
		if (AudioDevice)
		{
			// Upload the data to the hardware, but only if we've precached startup sounds already
			AudioDevice->Precache(this, false, false, false);
		}
	}

	// Only add this streaming sound if the platform supports streaming
	if (FApp::CanEverRenderAudio() && IsStreaming(nullptr) && FPlatformProperties::SupportsAudioStreaming())
	{
		// Only request loading the zeroth chunk when streaming is supported and we can render audio.
		// Especially important to avoid computing unneeded running platform related data during cook.
		LoadZerothChunk();
	}

#if WITH_EDITORONLY_DATA
	if (!SourceFilePath_DEPRECATED.IsEmpty() && AssetImportData)
	{
		FAssetImportInfo Info;
		Info.Insert(FAssetImportInfo::FSourceFile(SourceFilePath_DEPRECATED));
		AssetImportData->SourceData = MoveTemp(Info);
	}

#endif // #if WITH_EDITORONLY_DATA

#if WITH_EDITOR
	bNeedsThumbnailGeneration = true;
#endif // WITH_EDITOR

	INC_FLOAT_STAT_BY(STAT_AudioBufferTime, Duration);
	INC_FLOAT_STAT_BY(STAT_AudioBufferTimeChannels, NumChannels * Duration);

}

bool USoundWave::LoadZerothChunk()
{
	if (!ShouldUseStreamCaching())
	{
		return true;
	}

#if WITH_EDITOR 
	// Make sure the preparation has been done or push one async so it's hopefully ready when
	// we actually need the Zeroth Chunk
	CachePlatformData(true /* bAsyncCache */);

#else // WITH_EDITO
	// If the zeroth chunk is already loaded, early exit.
	if (GetWorkingSoundWaveData()->ZerothChunkData.GetView().Num() > 0)
	{
		return true;
	}

	if (GetWorkingSoundWaveData()->RunningPlatformData.Chunks.Num() <= 0)
	{
		return false;
	}

	// Otherwise, the zeroth chunk is cooked out to RunningPlatformData, and we just need to retrieve it.
	FStreamedAudioChunk& ZerothChunk = GetWorkingSoundWaveData()->RunningPlatformData.Chunks[0];
	// Some sanity checks to ensure that the bulk size set up
	UE_CLOGF(ZerothChunk.BulkData.GetBulkDataSize() != ZerothChunk.DataSize, LogAudio, Warning, "Bulk data serialized out had a mismatched size with the DataSize field. Soundwave: %ls Bulk Data Reported Size: %d Bulk Data Actual Size: %lld", *GetFullName(), ZerothChunk.DataSize, ZerothChunk.BulkData.GetBulkDataSize());

	GetWorkingSoundWaveData()->ZerothChunkData = ZerothChunk.BulkData.GetCopyAsBuffer(ZerothChunk.AudioDataSize, true);
#endif // WITH_EDITOR

	return true;
}

uint32 USoundWave::GetNumChunks() const
{
	return GetWorkingSoundWaveData()->RunningPlatformData.GetNumChunks();
}

uint32 USoundWave::GetSizeOfChunk(uint32 ChunkIndex) const
{
	check(ChunkIndex < GetNumChunks());

		return GetWorkingSoundWaveData()->RunningPlatformData.GetChunks()[ChunkIndex].AudioDataSize;
}

void USoundWave::GetChunkSizeStats(uint32& OutTotalBytesOfAudioData, uint32& OutMaxChunkBytesOfAudioData) const
{
	OutTotalBytesOfAudioData = 0;
	OutMaxChunkBytesOfAudioData = 0;

	
	for (const FStreamedAudioChunk& Chunk : GetWorkingSoundWaveData()->RunningPlatformData.GetChunks())
	{
		OutTotalBytesOfAudioData += Chunk.AudioDataSize;
		OutMaxChunkBytesOfAudioData = FMath::Max((uint32)Chunk.AudioDataSize, OutMaxChunkBytesOfAudioData);
	}
}

void USoundWave::BeginDestroy()
{
	Super::BeginDestroy();

	{
		UE::TUniqueLock Lock(SourcesPlayingCs);
		int32 CurrNumSourcesPlaying = SourcesPlaying.Num();

		for (int32 i = CurrNumSourcesPlaying - 1; i >= 0; --i)
		{
			ISoundWaveClient* SoundWaveClientPtr = SourcesPlaying[i];

			if (SoundWaveClientPtr && SoundWaveClientPtr->OnBeginDestroy(this))
			{
				// if OnBeginDestroy returned true, we are unsubscribing the SoundWaveClient...
				SourcesPlaying.RemoveAtSwap(i, EAllowShrinking::No);
			}
		}
	}

#if WITH_EDITOR
	// Flush any async results so we don't leak them in the DDC
	FlushAsyncLoadingDataFormats();
#endif
}

void USoundWave::InitAudioResource(FByteBulkData& CompressedData)
{
	LLM_SCOPE(ELLMTag::AudioSoundWaves);


	if (!GetWorkingSoundWaveData()->ResourceSize)
	{
		// Grab the compressed vorbis data from the bulk data
		GetWorkingSoundWaveData()->ResourceSize = CompressedData.GetBulkDataSize();
		if (GetWorkingSoundWaveData()->ResourceSize > 0)
		{
#if WITH_EDITOR
			uint8* TempDataPtr = nullptr;
			CompressedData.GetCopy((void**)&TempDataPtr, true);
			GetWorkingSoundWaveData()->ResourceData.Reset(TempDataPtr, GetWorkingSoundWaveData()->ResourceSize);
#else
			if (GetWorkingSoundWaveData()->ResourceData.GetView().GetData())
			{
				UE_LOGF(LogAudio, Display, "Soundwave '%ls' Has already had InitAudioResource() called, and taken ownership of it's compressed data."
					, *GetFullName());
				return;
			}
			else
			{
			}

			uint8* TempDataPtr = nullptr;
			if (CompressedData.IsDataMemoryMapped())
			{
				CompressedData.GetCopy((void**)&TempDataPtr, false);
			}
			else
			{
				FOwnedBulkDataPtr* OwnedBulkDataPtr = CompressedData.StealFileMapping();
				TempDataPtr = (uint8*)(OwnedBulkDataPtr)->GetPointer();
			}
			GetWorkingSoundWaveData()->ResourceData.Reset(TempDataPtr, GetWorkingSoundWaveData()->ResourceSize);
			if (!TempDataPtr)
			{
				UE_LOGF(LogAudio, Error, "Soundwave '%ls' was not loaded when it should have been, forcing a sync load.", *GetFullName());

				CompressedData.ForceBulkDataResident();
				if (CompressedData.IsDataMemoryMapped())
				{
					CompressedData.GetCopy((void**)&TempDataPtr, false);
				}
				else
				{
					FOwnedBulkDataPtr* OwnedBulkDataPtr = CompressedData.StealFileMapping();
					TempDataPtr = (uint8*)(OwnedBulkDataPtr)->GetPointer();
				}

				GetWorkingSoundWaveData()->ResourceData.Reset(TempDataPtr, GetWorkingSoundWaveData()->ResourceSize);
				if (!TempDataPtr)
				{
					UE_LOGF(LogAudio, Fatal, "Soundwave '%ls' failed to load even after forcing a sync load.", *GetFullName());
				}
			}
#endif
		}
	}
}

bool USoundWave::InitAudioResource(FName Format)
{
	LLM_SCOPE(ELLMTag::AudioSoundWaves);


	if (!GetWorkingSoundWaveData()->ResourceSize && (!FPlatformProperties::SupportsAudioStreaming() || !IsStreaming(nullptr)))
	{
		FByteBulkData* Bulk = GetCompressedData(Format, GetPlatformCompressionOverridesForCurrentPlatform());
		if (Bulk)
		{
#if WITH_EDITOR
			GetWorkingSoundWaveData()->ResourceSize = Bulk->GetBulkDataSize();
			check(GetWorkingSoundWaveData()->ResourceSize > 0);

			uint8* TempDataPtr = nullptr;
			Bulk->GetCopy((void**)&TempDataPtr, true);
			GetWorkingSoundWaveData()->ResourceData.Reset(TempDataPtr, GetWorkingSoundWaveData()->ResourceSize);

#else
			InitAudioResource(*Bulk);
			check(GetWorkingSoundWaveData()->ResourceSize > 0);
#endif
		}
		else
		{
			UE_LOGF(LogAudio, Warning, "Soundwave: %ls doesn't have compressed data! Format = %ls", *GetName(), *WriteToString<64>(Format));
		}
	}

	return GetWorkingSoundWaveData()->ResourceSize > 0;
}

void USoundWave::RemoveAudioResource()
{
	GetWorkingSoundWaveData()->ResourceSize = 0;
	GetWorkingSoundWaveData()->ResourceData.Empty();
}

#if WITH_EDITOR

void USoundWave::InvalidateSoundWaveIfNeccessary()
{
	if (bProcedural)
	{
		return;
	}

	// This will cause a recook of the sample rate overrides have changed since it last played
	int32 SampleRateOverride = (int32)GetSampleRateForCurrentPlatform();
	if (SampleRate != SampleRateOverride)
	{
		InvalidateCompressedData(true);
		SampleRate = SampleRateOverride;
	}
}

void USoundWave::PreSave(FObjectPreSaveContext InSaveContext)
{
	if (InSaveContext.IsCooking())
	{
		// Populate our cache ahead of Serialize, where we can't do any AssetRegistry queries safely.
		GetOwnerLoadingBehavior(InSaveContext.GetTargetPlatform());
	}
	Super::PreSave(InSaveContext);
}

void USoundWave::OnCookEvent(UE::Cook::ECookEvent CookEvent, UE::Cook::FCookEventContext& CookContext)
{
	Super::OnCookEvent(CookEvent, CookContext);
	if (CookEvent == UE::Cook::ECookEvent::PlatformCookDependencies && CookContext.IsCooking())
	{
		// We might need GetOwnerLoadingBehavior cached for RecordSoundWaveLoadingBehaviorDependenciesForCook
		GetOwnerLoadingBehavior(CookContext.GetTargetPlatform());
		UE::SoundWaveLoadingUtil::Private::RecordSoundWaveLoadingBehaviorDependenciesForCook(CookContext, this);
	}
}

ISoundWaveLoadingBehaviorUtil::FClassData USoundWave::GetOwnerLoadingBehavior(const ITargetPlatform* InTargetPlatform) const
{
	// Use {} or Name_None in the case of a missing platform.
	const FName PlatformName = InTargetPlatform ? *InTargetPlatform->PlatformName() : FName();

	// Lock for entirity of call.
	FScopeLock Lock(&OwnerLoadingBehaviorCacheCS);
	
	// We have it in our cache?
	if (const ISoundWaveLoadingBehaviorUtil::FClassData* Cached = OwnerLoadingBehaviorCache.Find(PlatformName))
	{
		return *Cached;
	}

	// ... otherwise, do lookup
	if (const ISoundWaveLoadingBehaviorUtil* Util = ISoundWaveLoadingBehaviorUtil::Get())
	{
		const ISoundWaveLoadingBehaviorUtil::FClassData Data = Util->FindOwningLoadingBehavior(this, InTargetPlatform);
		OwnerLoadingBehaviorCache.Add(PlatformName, Data);
		return Data;
	}
	
	// Not set.
	return {};
}

TArray<FSoundWaveCuePoint> USoundWave::GetCuePointsScaledForSampleRate(const float InSampleRate) const
{
	TArray<FSoundWaveCuePoint> CuePointsScaled;

	switch (CuePointOrigin)
	{
	case ESoundWaveCuePointOrigin::WaveFile:
		CuePointsScaled = CuePoints;
		break;
	case ESoundWaveCuePointOrigin::MarkerTransformation:
		CuePointsScaled = GetWaveformTransformationInfo().AllCuePointsRelativeToStartTime;
		break;
	default:
		CuePointsScaled = CuePoints;
		break;
	}

	ScaleCuePointsForSampleRate(InSampleRate, CuePointsScaled);

	return CuePointsScaled;
}

void USoundWave::ScaleCuePointsForSampleRate(const float InSampleRate, TArray<FSoundWaveCuePoint>& InOutCuePoints) const
{
	if (InOutCuePoints.IsEmpty() || InSampleRate <= 0.0f)
	{
		return;
	}

	// if ImportedSampleRate is 0, then it probably wasn't set properly yet in editor, 
	// this will be okay since SoundWave will update ImportedSampleRate on PostLoad
	// and re-initialize its internals so it will be valid.
	// ensuring or throwing an error/warning here isn't necessary
	if (ImportedSampleRate <= 0)
	{
		return;
	}

	float ResampleRatio = -1.0f;
	if (!FMath::IsNearlyEqual(InSampleRate, ImportedSampleRate))
	{
		ResampleRatio = InSampleRate / (float)ImportedSampleRate;
	}

	if (ResampleRatio > 0.0f)
	{
		for (FSoundWaveCuePoint& CuePoint : InOutCuePoints)
		{
			CuePoint.ScaleFrameValues(ResampleRatio);
		}
	}
}


float USoundWave::GetSampleRateForTargetPlatform(const ITargetPlatform* TargetPlatform)
{
	const FPlatformAudioCookOverrides* Overrides = FPlatformCompressionUtilities::GetCookOverrides(*TargetPlatform->IniPlatformName());
	if (Overrides)
	{
		return GetSampleRateForCompressionOverrides(Overrides);
	}
	else
	{
		return -1.0f;
	}
}

#if WITH_EDITOR

float USoundWave::GetSizeOfFirstAudioChunkInSeconds(const ITargetPlatform* InPlatform) const
{
#if WITH_EDITORONLY_DATA
	
	ensure(InPlatform);

	// This is the most important function for determining the size of chunk 1

	// We need to walk up the hierarchy of sound classes to first determine our FirstAudioChunkSize.
	if (GetWorkingSoundWaveData()->LoadingBehavior == ESoundWaveLoadingBehavior::Uninitialized)
	{
		CacheInheritedLoadingBehavior();
	}

	// Order of importance for where we get this setting.
	// Wave -> Owner -> Class

	// If Wave has a non-inherited Behavior set, use that.
	ISoundWaveLoadingBehaviorUtil::FClassData Behavior;
	if (LoadingBehavior != ESoundWaveLoadingBehavior::Inherited)
	{
		Behavior.LoadingBehavior = LoadingBehavior;
		Behavior.LengthOfFirstChunkInSeconds = SizeOfFirstAudioChunkInSeconds;
	}
	else // ... if we are inherited, try the owner first.
	{
		Behavior = GetOwnerLoadingBehavior(InPlatform);
	}

	// ... otherwise (or if we failed above) use this soundwave resolved behavior,
	if (Behavior.LoadingBehavior == ESoundWaveLoadingBehavior::Uninitialized)		
	{
		Behavior.LoadingBehavior = GetWorkingSoundWaveData()->LoadingBehavior;
		Behavior.LengthOfFirstChunkInSeconds = GetWorkingSoundWaveData()->SizeOfFirstAudioChunkInSeconds;
	}

	// Only give values if the behavior is Retain/Prime.
	if (Behavior.LoadingBehavior == ESoundWaveLoadingBehavior::RetainOnLoad ||
		Behavior.LoadingBehavior == ESoundWaveLoadingBehavior::PrimeOnLoad)
	{
		// See if we have an override for this platform on the SoundClass hierarchy.
		const FName PlatformName = InPlatform ? InPlatform->GetPlatformInfo().IniPlatformName : FName();
		const float PlatformSize = Behavior.LengthOfFirstChunkInSeconds.GetValueForPlatform(PlatformName);

		if (PlatformSize > 0)
		{
			return PlatformSize;
		}

		// If that's not set to anything fallback to setting on cook overrides, if there's anything set.
		if (const FPlatformAudioCookOverrides* CompressionOverrides = FPlatformCompressionUtilities::GetCookOverrides(*PlatformName.ToString()))
		{
			return CompressionOverrides->LengthOfFirstAudioChunkInSecs;
		}
	}
	
#endif //WITH_EDITORONLY_DATA

	// Nothing.
	return 0.f;
}

#endif //WITH_EDTIOR

void USoundWave::LogBakedData()
{
	const FString AnalysisPathName = *(FPaths::ProjectLogDir() + TEXT("BakedAudioAnalysisData/"));
	IFileManager::Get().MakeDirectory(*AnalysisPathName);

	FString SoundWaveName = FString::Printf(TEXT("%s.%s"), *FDateTime::Now().ToString(TEXT("%d-%H.%M.%S")), *GetName());

	if (CookedEnvelopeTimeData.Num())
	{
		FString EnvelopeFileName = FString::Printf(TEXT("%s.envelope.csv"), *SoundWaveName);
		FString FilenameFull = AnalysisPathName + EnvelopeFileName;

		FArchive* FileAr = IFileManager::Get().CreateDebugFileWriter(*FilenameFull);
		FOutputDeviceArchiveWrapper* FileArWrapper = new FOutputDeviceArchiveWrapper(FileAr);
		FOutputDevice* ReportAr = FileArWrapper;

		ReportAr->Log(TEXT("TimeStamp (Sec),Amplitude"));

		for (const FSoundWaveEnvelopeTimeData& EnvTimeData : CookedEnvelopeTimeData)
		{
			ReportAr->Logf(TEXT("%.4f,%.4f"), EnvTimeData.TimeSec, EnvTimeData.Amplitude);
		}

		// Shutdown and free archive resources
		FileArWrapper->TearDown();
		delete FileArWrapper;
		delete FileAr;
	}

	if (CookedSpectralTimeData.Num())
	{
		FString AnalysisFileName = FString::Printf(TEXT("%s.spectral.csv"), *SoundWaveName);
		FString FilenameFull = AnalysisPathName + AnalysisFileName;

		FArchive* FileAr = IFileManager::Get().CreateDebugFileWriter(*FilenameFull);
		FOutputDeviceArchiveWrapper* FileArWrapper = new FOutputDeviceArchiveWrapper(FileAr);
		FOutputDevice* ReportAr = FileArWrapper;

		// Build the header string
		FString ScratchString;
		ScratchString.Append(TEXT("Time Stamp (Sec),"));

		for (int32 i = 0; i < FrequenciesToAnalyze.Num(); ++i)
		{
			ScratchString.Append(FString::Printf(TEXT("%.2f Hz"), FrequenciesToAnalyze[i]));
			if (i != FrequenciesToAnalyze.Num() - 1)
			{
				ScratchString.Append(TEXT(","));
			}
		}

		ReportAr->Log(ScratchString);

		for (const FSoundWaveSpectralTimeData& SpectralTimeData : CookedSpectralTimeData)
		{
			ScratchString.Reset();
			ScratchString.Append(FString::Printf(TEXT("%.4f,"), SpectralTimeData.TimeSec));

			for (int32 i = 0; i < SpectralTimeData.Data.Num(); ++i)
			{
				ScratchString.Append(FString::Printf(TEXT("%.4f"), SpectralTimeData.Data[i].Magnitude));
				if (i != SpectralTimeData.Data.Num() - 1)
				{
					ScratchString.Append(TEXT(","));
				}
			}

			ReportAr->Log(*ScratchString);
		}

		// Shutdown and free archive resources
		FileArWrapper->TearDown();
		delete FileArWrapper;
		delete FileAr;
	}

}

static bool AnyFFTAnalysisPropertiesChanged(const FName& PropertyName)
{
	// List of properties which cause analysis to get triggered
	static FName OverrideSoundName = GET_MEMBER_NAME_CHECKED(USoundWave, OverrideSoundToUseForAnalysis);
	static FName EnableFFTAnalysisFName = GET_MEMBER_NAME_CHECKED(USoundWave, bEnableBakedFFTAnalysis);
	static FName FFTSizeFName = GET_MEMBER_NAME_CHECKED(USoundWave, FFTSize);
	static FName FFTAnalysisFrameSizeFName = GET_MEMBER_NAME_CHECKED(USoundWave, FFTAnalysisFrameSize);
	static FName FrequenciesToAnalyzeFName = GET_MEMBER_NAME_CHECKED(USoundWave, FrequenciesToAnalyze);
	static FName FFTAnalysisAttackTimeFName = GET_MEMBER_NAME_CHECKED(USoundWave, FFTAnalysisAttackTime);
	static FName FFTAnalysisReleaseTimeFName = GET_MEMBER_NAME_CHECKED(USoundWave, FFTAnalysisReleaseTime);

	return	PropertyName == OverrideSoundName ||
		PropertyName == EnableFFTAnalysisFName ||
		PropertyName == FFTSizeFName ||
		PropertyName == FFTAnalysisFrameSizeFName ||
		PropertyName == FrequenciesToAnalyzeFName ||
		PropertyName == FFTAnalysisAttackTimeFName ||
		PropertyName == FFTAnalysisReleaseTimeFName;
}

static bool AnyEnvelopeAnalysisPropertiesChanged(const FName& PropertyName)
{
	// List of properties which cause re-analysis to get triggered
	static FName OverrideSoundName = GET_MEMBER_NAME_CHECKED(USoundWave, OverrideSoundToUseForAnalysis);
	static FName EnableAmplitudeEnvelopeAnalysisFName = GET_MEMBER_NAME_CHECKED(USoundWave, bEnableAmplitudeEnvelopeAnalysis);
	static FName EnvelopeFollowerFrameSizeFName = GET_MEMBER_NAME_CHECKED(USoundWave, EnvelopeFollowerFrameSize);
	static FName EnvelopeFollowerAttackTimeFName = GET_MEMBER_NAME_CHECKED(USoundWave, EnvelopeFollowerAttackTime);
	static FName EnvelopeFollowerReleaseTimeFName = GET_MEMBER_NAME_CHECKED(USoundWave, EnvelopeFollowerReleaseTime);

	return	PropertyName == OverrideSoundName ||
		PropertyName == EnableAmplitudeEnvelopeAnalysisFName ||
		PropertyName == EnvelopeFollowerFrameSizeFName ||
		PropertyName == EnvelopeFollowerAttackTimeFName ||
		PropertyName == EnvelopeFollowerReleaseTimeFName;

}

void USoundWave::BakeFFTAnalysis()
{
	// Clear any existing spectral data regardless of if it's enabled. If this was enabled and is now toggled, this will clear previous data.
	CookedSpectralTimeData.Reset();

	// Perform analysis if enabled on the sound wave
	if (bEnableBakedFFTAnalysis)
	{
		// If there are no frequencies to analyze, we can't do the analysis
		if (!FrequenciesToAnalyze.Num())
		{
			UE_LOGF(LogAudio, Warning, "Soundwave '%ls' had baked FFT analysis enabled without specifying any frequencies to analyze.", *GetFullName());
			return;
		}

		if (ChannelSizes.Num() > 0)
		{
			UE_LOGF(LogAudio, Warning, "Soundwave '%ls' has multi-channel audio (channels greater than 2). Baking FFT analysis is not currently supported for this yet.", *GetFullName());
			return;
		}

		// Retrieve the raw imported data
		TArray<uint8> RawImportedWaveData;
		uint32 RawDataSampleRate = 0;
		uint16 RawDataNumChannels = 0;

		USoundWave* SoundWaveToUseForAnalysis = this;
		if (OverrideSoundToUseForAnalysis)
		{
			SoundWaveToUseForAnalysis = OverrideSoundToUseForAnalysis;
		}

		if (!SoundWaveToUseForAnalysis->GetImportedSoundWaveData(RawImportedWaveData, RawDataSampleRate, RawDataNumChannels))
		{
			return;
		}

		if (RawDataSampleRate == 0 || RawDataNumChannels == 0)
		{
			UE_LOGF(LogAudio, Error, "Failed to parse the raw imported data for '%ls' for baked FFT analysis.", *GetFullName());
			return;
		}

		const uint32 NumFrames = (RawImportedWaveData.Num() / sizeof(int16)) / RawDataNumChannels;
		int16* InputData = (int16*)RawImportedWaveData.GetData();

		Audio::FSpectrumAnalyzerSettings SpectrumAnalyzerSettings;
		switch (FFTSize)
		{
		case ESoundWaveFFTSize::VerySmall_64:
			SpectrumAnalyzerSettings.FFTSize = Audio::FSpectrumAnalyzerSettings::EFFTSize::Min_64;
			break;

		case ESoundWaveFFTSize::Small_256:
			SpectrumAnalyzerSettings.FFTSize = Audio::FSpectrumAnalyzerSettings::EFFTSize::Small_256;
			break;

		default:
		case ESoundWaveFFTSize::Medium_512:
			SpectrumAnalyzerSettings.FFTSize = Audio::FSpectrumAnalyzerSettings::EFFTSize::Medium_512;
			break;

		case ESoundWaveFFTSize::Large_1024:
			SpectrumAnalyzerSettings.FFTSize = Audio::FSpectrumAnalyzerSettings::EFFTSize::Large_1024;
			break;

		case ESoundWaveFFTSize::VeryLarge_2048:
			SpectrumAnalyzerSettings.FFTSize = Audio::FSpectrumAnalyzerSettings::EFFTSize::VeryLarge_2048;
			break;

		}

		// Prepare the spectral envelope followers
		Audio::FInlineEnvelopeFollowerInitParams EnvelopeFollowerInitParams;
		EnvelopeFollowerInitParams.SampleRate = static_cast<float>(RawDataSampleRate) / static_cast<float>(FMath::Max(1, FFTAnalysisFrameSize));
		EnvelopeFollowerInitParams.AttackTimeMsec = static_cast<float>(FFTAnalysisAttackTime);
		EnvelopeFollowerInitParams.ReleaseTimeMsec = static_cast<float>(FFTAnalysisReleaseTime);

		TArray<Audio::FInlineEnvelopeFollower> SpectralEnvelopeFollowers;
		for (int32 i = 0; i < FrequenciesToAnalyze.Num(); i++)
		{
			SpectralEnvelopeFollowers.Emplace(EnvelopeFollowerInitParams);
		}

		// Build a new spectrum analyzer
		Audio::FSpectrumAnalyzer SpectrumAnalyzer(SpectrumAnalyzerSettings, (float)RawDataSampleRate);

		// The audio data block to use to submit audio data to the spectrum analyzer
		Audio::FAlignedFloatBuffer AnalysisData;
		check(FFTAnalysisFrameSize > 256);
		AnalysisData.Reserve(FFTAnalysisFrameSize);

		float MaximumMagnitude = 0.0f;
		for (uint32 FrameIndex = 0; FrameIndex < NumFrames; ++FrameIndex)
		{
			// Get the averaged sample value of all the channels
			float SampleValue = 0.0f;
			for (uint16 ChannelIndex = 0; ChannelIndex < RawDataNumChannels; ++ChannelIndex)
			{
				SampleValue += (float)InputData[(FrameIndex * RawDataNumChannels) + ChannelIndex] / 32767.0f;
			}
			SampleValue /= RawDataNumChannels;

			// Accumate the samples in the scratch buffer
			AnalysisData.Add(SampleValue);

			// Until we reached the frame size
			if (AnalysisData.Num() == FFTAnalysisFrameSize)
			{
				SpectrumAnalyzer.PushAudio(AnalysisData.GetData(), AnalysisData.Num());

				// Block while the analyzer does the analysis
				SpectrumAnalyzer.PerformAnalysisIfPossible(true);

				FSoundWaveSpectralTimeData NewData;

				// Don't need to lock here since we're doing this sync, but it's here as that's the expected pattern for the Spectrum analyzer
				SpectrumAnalyzer.LockOutputBuffer();

				// Get the magntiudes for the specified frequencies
				for (int32 Index = 0; Index < FrequenciesToAnalyze.Num(); ++Index)
				{
					float Frequency = FrequenciesToAnalyze[Index];
					FSoundWaveSpectralDataEntry DataEntry;
					DataEntry.Magnitude = SpectrumAnalyzer.GetMagnitudeForFrequency(Frequency);

					// Feed the magnitude through the spectral envelope follower for this band
					DataEntry.Magnitude = SpectralEnvelopeFollowers[Index].ProcessSample(DataEntry.Magnitude);

					// Track the max magnitude so we can later set normalized magnitudes
					if (DataEntry.Magnitude > MaximumMagnitude)
					{
						MaximumMagnitude = DataEntry.Magnitude;
					}

					NewData.Data.Add(DataEntry);
				}

				SpectrumAnalyzer.UnlockOutputBuffer();

				// The time stamp is derived from the frame index and sample rate
				NewData.TimeSec = FMath::Max((float)(FrameIndex - FFTAnalysisFrameSize + 1) / RawDataSampleRate, 0.0f);

				/*
				// TODO: add FFTAnalysisTimeOffset
				// Don't let the time shift be more than the negative or postive duration
				float Duration = (float)NumFrames / RawDataSampleRate;
				float TimeShift = FMath::Clamp((float)FFTAnalysisTimeOffset / 1000, -Duration, Duration);

				NewData.TimeSec = NewData.TimeSec + (float)FFTAnalysisTimeOffset / 1000;

				// Truncate data if time shift is far enough to left that it's before the start of the sound
				if (TreatFileAsLoopingForAnalysis)
				{
					// Wrap the time value from endpoints if we're told this sound wave is looping
					if (NewData.TimeSec < 0.0f)
					{
						NewData.TimeSec = Duration + NewData.TimeSec;
					}
					else if (NewData.TimeSec >= Duration)
					{
						NewData.TimeSec = NewData.TimeSec - Duration;
					}
					CookedSpectralTimeData.Add(NewData);
				}
				else if (NewData.TimeSec > 0.0f)
				{
				CookedSpectralTimeData.Add(NewData);
				}
				*/

				CookedSpectralTimeData.Add(NewData);

				AnalysisData.Reset();
			}
		}

		// Sort predicate for sorting spectral data by time (lowest first)
		struct FSortSpectralDataByTime
		{
			FORCEINLINE bool operator()(const FSoundWaveSpectralTimeData& A, const FSoundWaveSpectralTimeData& B) const
			{
				return A.TimeSec < B.TimeSec;
			}
		};

		CookedSpectralTimeData.Sort(FSortSpectralDataByTime());

		// It's possible for the maximum magnitude to be 0.0 if the audio file was silent.
		if (MaximumMagnitude > 0.0f)
		{
			// Normalize all the magnitude values based on the highest magnitude
			for (FSoundWaveSpectralTimeData& SpectralTimeData : CookedSpectralTimeData)
			{
				for (FSoundWaveSpectralDataEntry& DataEntry : SpectralTimeData.Data)
				{
					DataEntry.NormalizedMagnitude = DataEntry.Magnitude / MaximumMagnitude;
				}
			}
		}

	}
}

void USoundWave::BakeEnvelopeAnalysis()
{
	// Clear any existing envelope data regardless of if it's enabled. If this was enabled and is now toggled, this will clear previous data.
	CookedEnvelopeTimeData.Reset();

	// Perform analysis if enabled on the sound wave
	if (bEnableAmplitudeEnvelopeAnalysis)
	{
		if (ChannelSizes.Num() > 0)
		{
			UE_LOGF(LogAudio, Warning, "Soundwave '%ls' has multi-channel audio (channels greater than 2). Baking envelope analysis is not currently supported for this yet.", *GetFullName());
			return;
		}

		// Retrieve the raw imported data
		TArray<uint8> RawImportedWaveData;
		uint32 RawDataSampleRate = 0;
		uint16 RawDataNumChannels = 0;

		USoundWave* SoundWaveToUseForAnalysis = this;
		if (OverrideSoundToUseForAnalysis)
		{
			SoundWaveToUseForAnalysis = OverrideSoundToUseForAnalysis;
		}

		if (!SoundWaveToUseForAnalysis->GetImportedSoundWaveData(RawImportedWaveData, RawDataSampleRate, RawDataNumChannels))
		{
			return;
		}

		if (RawDataSampleRate == 0 || RawDataNumChannels == 0)
		{
			UE_LOGF(LogAudio, Error, "Failed to parse the raw imported data for '%ls' for baked FFT analysis.", *GetFullName());
			return;
		}

		const uint32 NumFrames = (RawImportedWaveData.Num() / sizeof(int16)) / RawDataNumChannels;
		int16* InputData = (int16*)RawImportedWaveData.GetData();

		Audio::FInlineEnvelopeFollowerInitParams EnvelopeFollowerInitParams;
		EnvelopeFollowerInitParams.SampleRate = RawDataSampleRate; 
		EnvelopeFollowerInitParams.AttackTimeMsec = static_cast<float>(EnvelopeFollowerAttackTime);
		EnvelopeFollowerInitParams.ReleaseTimeMsec = static_cast<float>(EnvelopeFollowerReleaseTime);
		Audio::FInlineEnvelopeFollower EnvelopeFollower(EnvelopeFollowerInitParams);

		for (uint32 FrameIndex = 0; FrameIndex < NumFrames; ++FrameIndex)
		{
			// Get the averaged sample value of all the channels
			float SampleValue = 0.0f;
			for (uint16 ChannelIndex = 0; ChannelIndex < RawDataNumChannels; ++ChannelIndex)
			{
				SampleValue += (float)InputData[(FrameIndex * RawDataNumChannels) + ChannelIndex] / 32767.0f;
			}
			SampleValue /= RawDataNumChannels;

			float Output = EnvelopeFollower.ProcessSample(SampleValue);
			Output = FMath::Clamp(Output, 0.f, 1.f);
			


			// Until we reached the frame size
			if (FrameIndex % EnvelopeFollowerFrameSize == 0)
			{
				FSoundWaveEnvelopeTimeData NewData;
				NewData.Amplitude = Output;
				NewData.TimeSec = (float)FrameIndex / RawDataSampleRate;
				CookedEnvelopeTimeData.Add(NewData);
			}
		}
	}
}

void USoundWave::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	MaxDistance = ComputeMaxDistance();

	// Source buses and procedural sources do not have sample data and should 
	// avoid going through creation of streaming compressed data.
	const bool bHasSampleData = !(bIsSourceBus || bProcedural);
	if (!bHasSampleData)
	{
		return;
	}

	if (PropertyChangedEvent.Property == nullptr) 
	{
		// An empty event property field might mean the update comes from an undo or an import.
		// We are making an educated wish that an import task would still have async tasks running
		// when we get to the PostEditChange and an undo would not. This will help prevent us
		// from updating the asset twice on import, blocking the UI for the previous Update task
		// to complete. 
	
		if (IsAsyncWorkComplete())
		{
			UpdateAsset();
		}
	
		BakeFFTAnalysis();
		BakeEnvelopeAnalysis();

		return;
	}

	static const FName CompressionQualityFName = GET_MEMBER_NAME_CHECKED(USoundWave, CompressionQuality);
	static const FName SampleRateFName = GET_MEMBER_NAME_CHECKED(USoundWave,SampleRateQuality);
	static const FName StreamingFName = GET_MEMBER_NAME_CHECKED(USoundWave, bStreaming);
	static const FName SoundAssetCompressionTypeFName = GET_MEMBER_NAME_CHECKED(USoundWave, SoundAssetCompressionType);
	static const FName LoadingBehaviorFName = GET_MEMBER_NAME_CHECKED(USoundWave, LoadingBehavior);
	static const FName InitialChunkSizeFName = GET_MEMBER_NAME_CHECKED(USoundWave, InitialChunkSize_DEPRECATED);
	static const FName TransformationsFName = GET_MEMBER_NAME_CHECKED(USoundWave, Transformations);
	static const FName InlinedAudioInSecondsFName = GET_MEMBER_NAME_CHECKED(USoundWave, SizeOfFirstAudioChunkInSeconds);
	static const FName CloudStreamingFName = GET_MEMBER_NAME_CHECKED(USoundWave, bEnableCloudStreaming);
	static const FName CloudStreamingPlatformSettingsFName = GET_MEMBER_NAME_CHECKED(USoundWave, PlatformSettings);
	static const FName AssetUserDataFName = GET_MEMBER_NAME_CHECKED(USoundWave, AssetUserData);

	if (FProperty* PropertyThatChanged = PropertyChangedEvent.Property)
	{
		const FName& Name = PropertyThatChanged->GetFName();

		//Keep transformations UObjects configurations up to date
		if (Name == TransformationsFName)
		{
			UpdateTransformations();
		}
		else if (Name == AssetUserDataFName && !AssetUserData.IsEmpty())
		{
			// If a new Subtitle asset user datum was added, default its duration to the length of the SoundWave it was added to.
			// Look at the last item because a new user data added in the editor will always go at the end of the array- earlier data will have already had this operation applied.

			TObjectPtr<UAssetUserData>& LastModifiedUserData = AssetUserData.Last();
			if (IsValid(LastModifiedUserData))
			{
				USubtitleAssetUserData* UserDataAsSubtitle = Cast<USubtitleAssetUserData>(LastModifiedUserData.Get() );
				if (IsValid(UserDataAsSubtitle) && !UserDataAsSubtitle->Subtitles.IsEmpty())
				{
					FSubtitleAssetData& Subtitle = UserDataAsSubtitle->Subtitles.Last();
					if(Subtitle.Duration == SubtitleDefaultDuration && Subtitle.Text.IsEmpty())
					{
						// If the sound is looping indefinitely, the subtitle duration will default to the length of the first loop.
						// It is still editable by the user after this default is set.
						Subtitle.Duration = Duration;
					}
				}
			}
		}
	
		// Prevent constant re-compression of SoundWave while properties are being changed interactively
		if (PropertyChangedEvent.ChangeType != EPropertyChangeType::Interactive)
		{
		#if UE_USOUNDWAVE_HOT_RELOADING_ENABLED
			// Avoid modifying FSoundWaveProxy that is still in use by outside systems. 
			WorkingSoundWaveData = MakeShared<FSoundWaveData>();
		#endif // #if UE_USOUNDWAVE_HOT_RELOADING_ENABLED
			bool bSoundWaveDataNeedsInitialization = true;

			// Regenerate on save any compressed sound formats or if analysis needs to be re-done
			if (Name == LoadingBehaviorFName || Name == InlinedAudioInSecondsFName)
			{
				// Update and cache new loading behavior if it has changed. This
				// must be called before a new FSoundWaveProxy is created. 
				GetWorkingSoundWaveData()->LoadingBehavior = ESoundWaveLoadingBehavior::Uninitialized;
				CacheInheritedLoadingBehavior();
			}

			// Resolve conflicting properties: (force inline & force streaming)
			if (Name == LoadingBehaviorFName || Name == StreamingFName)
			{
				if ((bStreaming && LoadingBehavior == ESoundWaveLoadingBehavior::ForceInline))
				{
					bStreaming = false;
				}
			}

			if(Name == SoundAssetCompressionTypeFName)
			{
				SetSoundAssetCompressionType(SoundAssetCompressionType);
				bSoundWaveDataNeedsInitialization = false; //< SetSoundAssetCompressionType eventually calls InitializeDataFromSoundWave(...)
			}

			if (Name == TransformationsFName)
			{
				// For transformation changes, invalidate and rebuild compressed data without
				// stopping currently playing sounds. InvalidateCompressedData forks the
				// SoundWaveData so existing proxies (held by playing sounds) remain valid
				// while new playback requests will pick up the updated data.
				// Essentially UpdateAsset() without FreeResources() and FlushAudioRenderingCommands();

				InvalidateCompressedData();
				UpdatePlatformData();
				// Reset any error state we've encountered.
				ResetError();

				bSoundWaveDataNeedsInitialization = false; //< InvalidateCompressedData() eventually calls InitializeDataFromSoundWave(...)
			}
			else if (Name == CompressionQualityFName
				|| Name == SampleRateFName
				|| Name == StreamingFName
				|| Name == LoadingBehaviorFName
				|| Name == InlinedAudioInSecondsFName
				|| Name == InitialChunkSizeFName
				|| Name == CloudStreamingPlatformSettingsFName
				|| Name == CloudStreamingFName)
			{
				UpdateAsset();
				bSoundWaveDataNeedsInitialization = false; //< UpdateAsset() eventually calls InitializeDataFromSoundWave(...)
			}

			if (bSoundWaveDataNeedsInitialization)	
			{
				GetWorkingSoundWaveData()->InitializeDataFromSoundWave(*this);
			}

			if (AnyFFTAnalysisPropertiesChanged(Name))
			{
				BakeFFTAnalysis();
			}

			if (AnyEnvelopeAnalysisPropertiesChanged(Name))
			{
				BakeEnvelopeAnalysis();
			}
		}
	}
}

bool USoundWave::CanEditChange(const FProperty* InProperty) const
{
	if (!Super::CanEditChange(InProperty))
	{
		return false;
	}

	const FName& Name = InProperty->GetFName();
	if (Name == GetCloudStreamingEnabledPropertyName())
	{
		// If it is currently set then it is always editable so that it can be turned off if there is no suitable feature plugin available.
		if (IsCloudStreamingEnabled())
		{
			return true;
		}
		IModularFeatures::FScopedLockModularFeatureList ScopedLockModularFeatureList;
		TArray<Audio::ISoundWaveCloudStreamingFeature*> Features = IModularFeatures::Get().GetModularFeatureImplementations<Audio::ISoundWaveCloudStreamingFeature>(Audio::ISoundWaveCloudStreamingFeature::GetModularFeatureName());
		return Features.Num() > 0;
	}
	return true;
}


#endif // WITH_EDITOR

#if WITH_EDITORONLY_DATA
TArray<Audio::FTransformationPtr> USoundWave::CreateTransformations() const
{
	TArray<Audio::FTransformationPtr> TransformationPtrs;

	for(UWaveformTransformationBase* Transformation : Transformations)
	{
		if(Transformation)
		{
			TransformationPtrs.Add(Transformation->CreateTransformation());
		}
	}
	
	return TransformationPtrs;
}
#endif // WITH_EDITORONLY_DATA

void USoundWave::FreeResources(bool bStopSoundsUsingThisResource)
{
	check(IsInAudioThread());

	// Housekeeping of stats
	DEC_FLOAT_STAT_BY(STAT_AudioBufferTime, Duration);
	DEC_FLOAT_STAT_BY(STAT_AudioBufferTimeChannels, NumChannels * Duration);

	// GEngine is NULL during script compilation and GEngine->Client and its audio device might be
	// destroyed first during the exit purge.
	if (GEngine && !GExitPurge)
	{
		// Notify the audio device to free the bulk data associated with this wave.
		FAudioDeviceManager* AudioDeviceManager = GEngine->GetAudioDeviceManager();
		if (bStopSoundsUsingThisResource && AudioDeviceManager)
		{
			AudioDeviceManager->StopSoundsUsingResource(this);
			AudioDeviceManager->FreeResource(this);
		}
	}

	if (CachedRealtimeFirstBuffer)
	{
		FMemory::Free(CachedRealtimeFirstBuffer);
		CachedRealtimeFirstBuffer = nullptr;
	}

	// Just in case the data was created but never uploaded
	if (RawPCMData)
	{
		FMemory::Free(RawPCMData);
		RawPCMData = nullptr;
	}

	// Stat housekeeping
	DEC_DWORD_STAT_BY(STAT_AudioMemorySize, TrackedMemoryUsage);
	DEC_DWORD_STAT_BY(STAT_AudioMemory, TrackedMemoryUsage);
	TrackedMemoryUsage = 0;

	ResourceID = 0;
	bDynamicResource = false;
	DecompressionType = DTYPE_Setup;
	SetPrecacheState(ESoundWavePrecacheState::NotStarted);
	bDecompressedFromOgg = false;

	if (ResourceState == ESoundWaveResourceState::Freeing)
	{
		ResourceState = ESoundWaveResourceState::Freed;
	}
}

bool USoundWave::CleanupDecompressor(bool bForceWait)
{
	check(IsInAudioThread());

	if (!AudioDecompressor)
	{
		check(GetPrecacheState() == ESoundWavePrecacheState::Done);
		return true;
	}

	if (AudioDecompressor->IsDone())
	{
		delete AudioDecompressor;
		AudioDecompressor = nullptr;
		SetPrecacheState(ESoundWavePrecacheState::Done);
		return true;
	}

	if (bForceWait)
	{
		AudioDecompressor->EnsureCompletion();
		delete AudioDecompressor;
		AudioDecompressor = nullptr;
		SetPrecacheState(ESoundWavePrecacheState::Done);
		return true;
	}

	return false;
}

FWaveInstance& USoundWave::HandleStart(FActiveSound& ActiveSound, const UPTRINT WaveInstanceHash) const
{
	// Create a new wave instance and associate with the ActiveSound
	FWaveInstance& WaveInstance = ActiveSound.AddWaveInstance(WaveInstanceHash);

	// Add in the subtitle if they exist
	if (ActiveSound.bHandleSubtitles && Subtitles.Num() > 0)
	{
		FQueueSubtitleParams QueueSubtitleParams(Subtitles);
		{
			QueueSubtitleParams.AudioComponentID = ActiveSound.GetAudioComponentID();
			QueueSubtitleParams.WorldPtr = ActiveSound.GetWeakWorld();
			QueueSubtitleParams.WaveInstance = (PTRINT)&WaveInstance;
			QueueSubtitleParams.SubtitlePriority = ActiveSound.SubtitlePriority;
			QueueSubtitleParams.Duration = Duration;
			QueueSubtitleParams.bManualWordWrap = bManualWordWrap;
			QueueSubtitleParams.bSingleLine = bSingleLine;
			QueueSubtitleParams.RequestedStartTime = ActiveSound.RequestedStartTime;
		}

		FSubtitleManager::QueueSubtitles(QueueSubtitleParams);
	}

	return WaveInstance;
}

bool USoundWave::IsReadyForFinishDestroy()
{
	{
		UE::TUniqueLock Lock(SourcesPlayingCs);

		for (ISoundWaveClient* SoundWaveClientPtr : SourcesPlaying)
		{
			// Should never have a null element here, inactive elements are removed from the container
			check(SoundWaveClientPtr);
			SoundWaveClientPtr->OnIsReadyForFinishDestroy(this);
		}
	}

	{
		UE::TUniqueLock Lock(SourcesPlayingCs);

		for (ISoundWaveClient* SoundWaveClientPtr : SourcesPlaying)
		{
			// Should never have a null element here, inactive elements are removed from the container
			check(SoundWaveClientPtr);
			SoundWaveClientPtr->OnIsReadyForFinishDestroy(this);
		}
	}

	check(GetPrecacheState() != ESoundWavePrecacheState::InProgress);

	// Wait till streaming and decompression finishes before deleting resource.
	if (ResourceState == ESoundWaveResourceState::NeedsFree)
	{
		DECLARE_CYCLE_STAT(TEXT("FAudioThreadTask.FreeResources"), STAT_AudioFreeResources, STATGROUP_AudioThreadCommands);

		USoundWave* SoundWave = this;
		ResourceState = ESoundWaveResourceState::Freeing;
		FAudioThread::RunCommandOnAudioThread([SoundWave]()
		{
			SoundWave->FreeResources();
		}, GET_STATID(STAT_AudioFreeResources));
	}

	return ResourceState == ESoundWaveResourceState::Freed;
}


void USoundWave::FinishDestroy()
{
	Super::FinishDestroy();

	{
		UE::TUniqueLock Lock(SourcesPlayingCs);
		int32 CurrNumSourcesPlaying = SourcesPlaying.Num();

		for (int32 i = CurrNumSourcesPlaying - 1; i >= 0; --i)
		{
			ISoundWaveClient* SoundWaveClientPtr = SourcesPlaying[i];

			if (SoundWaveClientPtr)
			{
				SoundWaveClientPtr->OnFinishDestroy(this);
				SourcesPlaying.RemoveAtSwap(i, EAllowShrinking::No);
			}
		}
	}

	check(GetPrecacheState() != ESoundWavePrecacheState::InProgress);
	check(AudioDecompressor == nullptr);

#if WITH_EDITOR
	if (!GExitPurge)
	{
		ClearAllCachedCookedPlatformData();
	}
#endif
}

void USoundWave::Parse(FAudioDevice* AudioDevice, const UPTRINT NodeWaveInstanceHash, FActiveSound& ActiveSound, const FSoundParseParameters& ParseParams, TArray<FWaveInstance*>& WaveInstances)
{
	check(AudioDevice);

	FWaveInstance* WaveInstance = ActiveSound.FindWaveInstance(NodeWaveInstanceHash);

	const bool bIsNewWave = WaveInstance == nullptr;

	// Create a new WaveInstance if this SoundWave doesn't already have one associated with it.
	if (!WaveInstance)
	{
		if (!ActiveSound.bRadioFilterSelected)
		{
			ActiveSound.ApplyRadioFilter(ParseParams);
		}

		WaveInstance = &HandleStart(ActiveSound, NodeWaveInstanceHash);
	}
	else
	{
		// If WaveInstance is finished & sound is procedurally generated and has
		// virtualization disabled, early out to stop playback at the ActiveSound level.
		const bool bVirtualizationDisabled = VirtualizationMode == EVirtualizationMode::Disabled;
		const bool bInstanceIsFinished = WaveInstance->bIsFinished;
		const bool bFinishedNotify = bInstanceIsFinished && WaveInstance->bAlreadyNotifiedHook;
		if (bFinishedNotify && bProcedural && bVirtualizationDisabled)
		{
			return;
		}
	}

	// Looping sounds are never actually finished
	if (!IsOneShot() || ParseParams.bLooping)
	{
		WaveInstance->bIsFinished = false;

		const USoundBase* Sound = ActiveSound.GetSound();

		if (!Sound->IsOneShot())
		{
#if !(NO_LOGGING || UE_BUILD_SHIPPING || UE_BUILD_TEST)
			if (!ActiveSound.bWarnedAboutOrphanedLooping && ActiveSound.GetAudioComponentID() == 0 && ActiveSound.FadeOut == FActiveSound::EFadeOut::None)
			{
				UE_LOGF(LogAudio, Warning, "Detected orphaned sound which is not a one-shot '%ls'. Sounds which are not one-shots need to have audio components or they risk being orphaned.", *ActiveSound.GetSound()->GetName());
				ActiveSound.bWarnedAboutOrphanedLooping = true;
			}
#endif
		}
	}

	// Early out if finished.
	if (WaveInstance->bIsFinished)
	{
		return;
	}

	// Propagate properties and add WaveInstance to outgoing array of FWaveInstances.
	WaveInstance->SetVolume(ParseParams.Volume * Volume);
	WaveInstance->SetVolumeMultiplier(ParseParams.VolumeMultiplier);
	WaveInstance->SetDistanceAttenuation(ParseParams.DistanceAttenuation);
	WaveInstance->SetOcclusionAttenuation(ParseParams.OcclusionAttenuation);
	WaveInstance->SetPitch(ParseParams.Pitch * Pitch);
	WaveInstance->bEnableLowPassFilter = ParseParams.bEnableLowPassFilter;
	WaveInstance->bEnableHighPassFilter = ParseParams.bEnableHighPassFilter;
	WaveInstance->bIsOccluded = ParseParams.bIsOccluded;
	WaveInstance->LowPassFilterFrequency = ParseParams.LowPassFilterFrequency;
	WaveInstance->HighPassFilterFrequency = ParseParams.HighPassFilterFrequency;
	WaveInstance->OcclusionFilterFrequency = ParseParams.OcclusionFilterFrequency;
	WaveInstance->AttenuationLowpassFilterFrequency = ParseParams.AttenuationLowpassFilterFrequency;
	WaveInstance->AttenuationHighpassFilterFrequency = ParseParams.AttenuationHighpassFilterFrequency;
	WaveInstance->AmbientZoneFilterFrequency = ParseParams.AmbientZoneFilterFrequency;
	WaveInstance->bApplyRadioFilter = ActiveSound.bApplyRadioFilter;

	WaveInstance->StartTime = ParseParams.StartTime;
	// Override the start time for seek restart virtualization mode
	if (ActiveSound.SeekRestartTimes.Contains(WaveInstance->WaveInstanceHash))
	{
		float SeekRestartTime = ActiveSound.SeekRestartTimes[WaveInstance->WaveInstanceHash];
		// For sounds with a defined duration, we wrap the seek time so that it sounds like it properly looped.
		// Procedural sounds will get the unwrapped time, which they can use how they see fit.
		const bool bWrapSeekTime = !bProcedural && (ParseParams.bLooping || IsLooping());
		if (bWrapSeekTime)
		{
			WaveInstance->StartTime = FMath::Fmod(SeekRestartTime, Duration);
		}
		else
		{
			WaveInstance->StartTime = SeekRestartTime;
		}
	}

	WaveInstance->UserIndex = ActiveSound.UserIndex;
	WaveInstance->NonSpatializedRadiusStart = ParseParams.NonSpatializedRadiusStart;
	WaveInstance->NonSpatializedRadiusEnd = ParseParams.NonSpatializedRadiusEnd;
	WaveInstance->NonSpatializedRadiusMode = ParseParams.NonSpatializedRadiusMode;
	WaveInstance->StereoSpread = ParseParams.StereoSpread;
	WaveInstance->AttenuationDistance = ParseParams.AttenuationDistance;
	WaveInstance->ListenerToSoundDistance = ParseParams.ListenerToSoundDistance;
	WaveInstance->ListenerToSoundDistanceForPanning = ParseParams.ListenerToSoundDistanceForPanning;
	WaveInstance->AbsoluteAzimuth = ParseParams.AbsoluteAzimuth;

	if (ParseParams.SourceEffectChain)
	{
		// Only copy the source effect chain pointer if it supports the channel count
		if (ParseParams.SourceEffectChain->SupportsChannelCount(NumChannels))
		{
			WaveInstance->SourceEffectChain = ParseParams.SourceEffectChain;
		}
		else
		{
			WaveInstance->SourceEffectChain = nullptr;
			UE_LOGF(LogAudio, Verbose, "Sound %ls from Audio Component %ls has a source effect chain but an incompatible channel count %d, so will not be instantiated", 
				ActiveSound.GetSound() ? *ActiveSound.GetSound()->GetFName().ToString() : TEXT("None"),
				*ActiveSound.GetAudioComponentName(),
				NumChannels);
		}
	}

	bool bAlwaysPlay = false;

	// Properties from the sound class
	WaveInstance->SoundClass = ParseParams.SoundClass;
	bool bFoundSoundClass = false;
	if (ParseParams.SoundClass)
	{
		const FSoundClassProperties* SoundClassProperties = AudioDevice->GetSoundClassCurrentProperties(ParseParams.SoundClass);
		if (!SoundClassProperties)
		{
			UE_LOGF(LogAudio, Warning, "Unable to find sound class properties for sound class %ls", *ParseParams.SoundClass->GetName());
		}
		else
		{
			bFoundSoundClass = true;

			// Use values from "parsed/ propagated" sound class properties
			float VolumeMultiplier = WaveInstance->GetVolumeMultiplier();
			WaveInstance->SetVolumeMultiplier(VolumeMultiplier * SoundClassProperties->Volume);
			WaveInstance->SetPitch(WaveInstance->Pitch * SoundClassProperties->Pitch);

			WaveInstance->SoundClassFilterFrequency = SoundClassProperties->LowPassFilterFrequency;
			WaveInstance->VoiceCenterChannelVolume = SoundClassProperties->VoiceCenterChannelVolume;
			WaveInstance->RadioFilterVolume = SoundClassProperties->RadioFilterVolume * ParseParams.VolumeMultiplier;
			WaveInstance->RadioFilterVolumeThreshold = SoundClassProperties->RadioFilterVolumeThreshold * ParseParams.VolumeMultiplier;
			WaveInstance->LFEBleed = SoundClassProperties->LFEBleed;

			WaveInstance->bIsUISound = ActiveSound.bIsUISound || SoundClassProperties->bIsUISound;
			WaveInstance->bIsMusic = ActiveSound.bIsMusic || SoundClassProperties->bIsMusic;
			WaveInstance->bCenterChannelOnly = ActiveSound.bCenterChannelOnly || SoundClassProperties->bCenterChannelOnly;
			WaveInstance->bReverb = ActiveSound.bReverb || SoundClassProperties->bReverb;
			WaveInstance->OutputTarget = SoundClassProperties->OutputTarget;

			if (SoundClassProperties->bApplyEffects)
			{
				const UAudioSettings* Settings = GetDefault<UAudioSettings>();
				if (LIKELY(IsValid(Settings)))
				{
					WaveInstance->SoundSubmix = Cast<USoundSubmix>(FSoftObjectPtr(Settings->EQSubmix).Get());
				}
			}
			else if (SoundClassProperties->DefaultSubmix)
			{
				WaveInstance->SoundSubmix = SoundClassProperties->DefaultSubmix;
			}

			if (SoundClassProperties->bApplyAmbientVolumes)
			{
				VolumeMultiplier = WaveInstance->GetVolumeMultiplier();
				WaveInstance->SetVolumeMultiplier(VolumeMultiplier * ParseParams.InteriorVolumeMultiplier);
				WaveInstance->RadioFilterVolume *= ParseParams.InteriorVolumeMultiplier;
				WaveInstance->RadioFilterVolumeThreshold *= ParseParams.InteriorVolumeMultiplier;
			}

			bAlwaysPlay = ActiveSound.bAlwaysPlay || SoundClassProperties->bAlwaysPlay;
		}
	}

	if (!bFoundSoundClass)
	{
		WaveInstance->VoiceCenterChannelVolume = 0.f;
		WaveInstance->RadioFilterVolume = 0.f;
		WaveInstance->RadioFilterVolumeThreshold = 0.f;
		WaveInstance->LFEBleed = 0.f;
		WaveInstance->bIsUISound = ActiveSound.bIsUISound;
		WaveInstance->bIsMusic = ActiveSound.bIsMusic;
		WaveInstance->bReverb = ActiveSound.bReverb;
		WaveInstance->bCenterChannelOnly = ActiveSound.bCenterChannelOnly;

		bAlwaysPlay = ActiveSound.bAlwaysPlay;
	}

	WaveInstance->bIsAmbisonics = bIsAmbisonics;

	if (ParseParams.SoundSubmix)
	{
		WaveInstance->SoundSubmix = ParseParams.SoundSubmix;
	}
	else if (USoundSubmixBase* WaveSubmix = GetSoundSubmix())
	{
		WaveInstance->SoundSubmix = WaveSubmix;
	}

	// If set to bAlwaysPlay, increase the current sound's priority scale by 10x. This will still result in a possible 0-priority output if the sound has 0 actual volume
	if (bAlwaysPlay)
	{
		static constexpr float VolumeWeightedMaxPriority = TNumericLimits<float>::Max() / MAX_VOLUME;
		WaveInstance->Priority = VolumeWeightedMaxPriority;
	}
	else
	{
		WaveInstance->Priority = FMath::Clamp(ParseParams.Priority, 0.0f, 100.0f);
	}

	WaveInstance->Location = ParseParams.Transform.GetTranslation();
	WaveInstance->bIsStarted = true;
	WaveInstance->bAlreadyNotifiedHook = false;

	WaveInstance->WaveData = this;
	WaveInstance->NotifyBufferFinishedHooks = ParseParams.NotifyBufferFinishedHooks;
	WaveInstance->LoopingMode = ((!IsOneShot() || ParseParams.bLooping) ? LOOP_Forever : LOOP_Never);
	WaveInstance->bIsPaused = ParseParams.bIsPaused;

	// If we're normalizing 3d stereo spatialized sounds, we need to scale by -6 dB
	WaveInstance->SetUseSpatialization(ParseParams.bUseSpatialization);

	// Setup the spat method if we're actually spatializing (note a cvar can turn this off so we use the getter here)
	if (WaveInstance->GetUseSpatialization())
	{
		WaveInstance->SpatializationMethod = ParseParams.SpatializationMethod;

		// Check for possible HRTF-enforcement if this is a spatialized sound
		if (AudioDevice->IsHRTFEnabledForAll() && ParseParams.SpatializationMethod == ESoundSpatializationAlgorithm::SPATIALIZATION_Default)
		{
			WaveInstance->SpatializationMethod = ESoundSpatializationAlgorithm::SPATIALIZATION_HRTF;
		}
		else
		{
			WaveInstance->SpatializationMethod = ParseParams.SpatializationMethod;
		}

		//If this is using binaural audio, update whether its an external send
		if (WaveInstance->SpatializationMethod == ESoundSpatializationAlgorithm::SPATIALIZATION_HRTF)
		{
			WaveInstance->SetSpatializationIsExternalSend(ParseParams.bSpatializationIsExternalSend);
		}

		// Apply stereo normalization to wave instances if enabled
		if (ParseParams.bApplyNormalizationToStereoSounds && NumChannels == 2)
		{
			float WaveInstanceVolume = WaveInstance->GetVolume();
			WaveInstance->SetVolume(WaveInstanceVolume * 0.5f);
		}
	}

	// Update reverb send levels. 
	WaveInstance->bReverb = WaveInstance->bReverb && !WaveInstance->bIsMusic && (AllowReverbForMultichannelSources || WaveInstance->WaveData->NumChannels <= 2);
	if (WaveInstance->bReverb)
	{
		if (ParseParams.ReverbSendMethod == EReverbSendMethod::Manual)
		{
			WaveInstance->ReverbSendLevel = FMath::Clamp(ParseParams.ManualReverbSendLevel, 0.0f, 1.0f);
		}
		else
		{
			// The alpha value is determined identically between manual and custom curve methods
			const FVector2D& ReverbSendRadialRange = ParseParams.ReverbSendLevelDistanceRange;
			const float Denom = FMath::Max(ReverbSendRadialRange.Y - ReverbSendRadialRange.X, 1.0f);
			const float Alpha = FMath::Clamp((WaveInstance->ListenerToSoundDistance - ReverbSendRadialRange.X) / Denom, 0.0f, 1.0f);

			if (ParseParams.ReverbSendMethod == EReverbSendMethod::Linear)
			{
				WaveInstance->ReverbSendLevel = FMath::Clamp(FMath::Lerp(ParseParams.ReverbSendLevelRange.X, ParseParams.ReverbSendLevelRange.Y, Alpha), 0.0f, 1.0f);
			}
			else
			{
				WaveInstance->ReverbSendLevel = FMath::Clamp(ParseParams.CustomReverbSendCurve.GetRichCurveConst()->Eval(Alpha), 0.0f, 1.0f);
			}
		}
	}

 	// Copy the submix sends. (both attenuation and soundbase versions).
 	WaveInstance->AttenuationSubmixSends = ParseParams.AttenuationSubmixSends;
	WaveInstance->SoundSubmixSends = ParseParams.SoundSubmixSends;

	// Get the envelope follower settings
	WaveInstance->EnvelopeFollowerAttackTime = ParseParams.EnvelopeFollowerAttackTime;
	WaveInstance->EnvelopeFollowerReleaseTime = ParseParams.EnvelopeFollowerReleaseTime;

	// Copy over the source bus send and data
	if (!WaveInstance->ActiveSound->bIsPreviewSound)
	{
		//Parse the parameters of the wave instance
		WaveInstance->bEnableBusSends = ParseParams.bEnableBusSends;

		// HRTF rendering doesn't render their output on the base submix
		if (!((WaveInstance->SpatializationMethod == SPATIALIZATION_HRTF) && (WaveInstance->bSpatializationIsExternalSend)))
		{
			if (ActiveSound.bHasActiveMainSubmixOutputOverride)
			{
				WaveInstance->bEnableBaseSubmix = ActiveSound.bEnableMainSubmixOutputOverride;
			}
			else
			{
				WaveInstance->bEnableBaseSubmix = ParseParams.bEnableBaseSubmix;
			}
		}
		else
		{
			WaveInstance->bEnableBaseSubmix = false;
		}
		WaveInstance->bEnableSubmixSends = ParseParams.bEnableSubmixSends;

		// Active sounds can override their enablement behavior via audio components
		if (ActiveSound.bHasActiveBusSendRoutingOverride)
		{
			WaveInstance->bEnableBusSends = ActiveSound.bEnableBusSendRoutingOverride;
		}

		if (ActiveSound.bHasActiveSubmixSendRoutingOverride)
		{
			WaveInstance->bEnableSubmixSends = ActiveSound.bEnableSubmixSendRoutingOverride;
		}
	}
	else //if this is a preview sound, ignore Submix sends
	{
#if WITH_EDITORONLY_DATA
		// Only allow submix sends if this is a metasound preview
		if (!WaveInstance->ActiveSound->bIsDryPreviewPlayback)
		{
#endif //WITH_EDITORONLY_DATA
			WaveInstance->bEnableSubmixSends = ParseParams.bEnableSubmixSends;
			WaveInstance->bEnableBaseSubmix = ParseParams.bEnableBaseSubmix;
#if WITH_EDITORONLY_DATA
		}
		else
		{
			WaveInstance->bEnableBaseSubmix = true;
		}
#endif //WITH_EDITORONLY_DATA

		WaveInstance->bEnableBusSends = ParseParams.bEnableBusSends;
	}

	for (int32 BusSendType = 0; BusSendType < (int32)EBusSendType::Count; ++BusSendType)
	{
		WaveInstance->BusSends[BusSendType] = ParseParams.BusSends[BusSendType];
	}

	// Pass along plugin settings to the wave instance
	WaveInstance->SpatializationPluginSettings = ParseParams.SpatializationPluginSettings;
	WaveInstance->OcclusionPluginSettings = ParseParams.OcclusionPluginSettings;
	WaveInstance->ReverbPluginSettings = ParseParams.ReverbPluginSettings;
	WaveInstance->SourceDataOverridePluginSettings = ParseParams.SourceDataOverridePluginSettings;

	WaveInstance->bEnableSourceDataOverride = ParseParams.bEnableSourceDataOverride;

	WaveInstance->bShouldUseAudioLink = ParseParams.bEnableSendToAudioLink;
	WaveInstance->AudioLinkSettingsOverride = ParseParams.AudioLinkSettingsOverride;

	const bool bIsWavePlaying = WaveInstance->IsPlaying();
	if (bIsWavePlaying)
	{
		WaveInstances.Add(WaveInstance);
		ActiveSound.bFinished = false;
	}
	else if (WaveInstance->LoopingMode == LOOP_Forever)
	{
		ActiveSound.bFinished = false;
	}
	// Not looping, silent, and not set to play when silent
	else
	{
		// If no wave instance added to transient array not looping, and just created, immediately delete
		// to avoid initializing on a later tick (achieved by adding to active sound's wave instance map
		// but not the passed transient WaveInstance array)
		if (bIsNewWave)
		{
			ActiveSound.RemoveWaveInstance(NodeWaveInstanceHash);
			return;
		}
	}

#if UE_AUDIO_PROFILERTRACE_ENABLED
	const bool bChannelEnabled = UE_TRACE_CHANNELEXPR_IS_ENABLED(AudioMixerChannel);
	const bool bWaveIsActive = bIsWavePlaying || !ActiveSound.bFinished;

	if (bChannelEnabled && bWaveIsActive && WaveInstance->SoundClass != nullptr)
	{
		UE_TRACE_LOG(Audio, SoundPriority, AudioMixerChannel)
			<< SoundPriority.DeviceId(AudioDevice->DeviceID)
			<< SoundPriority.Timestamp(FPlatformTime::Cycles64())
			<< SoundPriority.PlayOrder(WaveInstance->GetPlayOrder())
			<< SoundPriority.ActiveSoundPlayOrder(ActiveSound.GetPlayOrder())
			<< SoundPriority.Priority(WaveInstance->Priority);

		UE_TRACE_LOG(Audio, SoundDistance, AudioMixerChannel)
			<< SoundDistance.DeviceId(AudioDevice->DeviceID)
			<< SoundDistance.Timestamp(FPlatformTime::Cycles64())
			<< SoundDistance.PlayOrder(WaveInstance->GetPlayOrder())
			<< SoundDistance.ActiveSoundPlayOrder(ActiveSound.GetPlayOrder())
			<< SoundDistance.Distance(WaveInstance->ListenerToSoundDistance);
	}
#endif // UE_AUDIO_PROFILERTRACE_ENABLED

#if !NO_LOGGING
	// Sanity check
	if (NumChannels > 2 && !WaveInstance->bIsAmbisonics && WaveInstance->GetUseSpatialization() && !WaveInstance->bReportedSpatializationWarning)
	{
		static TSet<USoundWave*> ReportedSounds;
		if (!ReportedSounds.Contains(this))
		{
			FString SoundWarningInfo = FString::Printf(TEXT("Spatialization on sounds with channels greater than 2 is not supported. SoundWave: %s"), *GetName());
			if (ActiveSound.GetSound() != this)
			{
				SoundWarningInfo += FString::Printf(TEXT(" SoundCue: %s"), *ActiveSound.GetSound()->GetName());
			}

			const uint64 AudioComponentID = ActiveSound.GetAudioComponentID();
			if (AudioComponentID > 0)
			{
				FAudioThread::RunCommandOnGameThread([AudioComponentID, SoundWarningInfo]()
				{
					if (UAudioComponent* AudioComponent = UAudioComponent::GetAudioComponentFromID(AudioComponentID))
					{
						AActor* SoundOwner = AudioComponent->GetOwner();
						UE_LOGF(LogAudio, Verbose, "%ls Actor: %ls AudioComponent: %ls", *SoundWarningInfo, (SoundOwner ? *SoundOwner->GetName() : TEXT("None")), *AudioComponent->GetName());
					}
					else
					{
						UE_LOGF(LogAudio, Verbose, "%ls", *SoundWarningInfo);
					}
				});
			}
			else
			{
				UE_LOGF(LogAudio, Verbose, "%ls", *SoundWarningInfo);
			}

			ReportedSounds.Add(this);
		}
		WaveInstance->bReportedSpatializationWarning = true;
	}
#endif // !NO_LOGGING
}

bool USoundWave::IsPlayable() const
{
	return true;
}

float USoundWave::GetDuration() const
{
	return (bLooping ? INDEFINITELY_LOOPING_DURATION : Duration);
}

bool USoundWave::IsStreaming(const TCHAR* PlatformName/* = nullptr */) const
{
	const bool bIsForceInline = LoadingBehavior == ESoundWaveLoadingBehavior::ForceInline;
	const bool bIsForcedNonStreamingInEditor = (GIsEditor && ForceNonStreamingInEditorCVar != 0);

	const bool bIsForcedNonStreaming = (bProcedural || bIsForceInline || bIsForcedNonStreamingInEditor);

	if (bIsForcedNonStreaming)
	{
		GetWorkingSoundWaveData()->bIsStreaming = false;
	}
	else
	{
		GetWorkingSoundWaveData()->bIsStreaming = IsStreaming(*FPlatformCompressionUtilities::GetCookOverrides(PlatformName));
	}
	
	return GetWorkingSoundWaveData()->bIsStreaming;
}

bool USoundWave::IsStreaming(const FPlatformAudioCookOverrides& Overrides) const
{
	
	// We stream if (A) bStreaming is set to true, (B) bForceInline is false and either bUseLoadOnDemand was set to true in
	// our cook overrides, or the AutoStreamingThreshold was set and this sound is longer than the auto streaming threshold.
	const bool bIsForceInline = LoadingBehavior == ESoundWaveLoadingBehavior::ForceInline;
	const bool bIsForcedNonStreamingInEditor = (GIsEditor && ForceNonStreamingInEditorCVar != 0);

	const bool bIsForcedNonStreaming = (bProcedural || bIsForceInline || bIsForcedNonStreamingInEditor);

	if (bIsForcedNonStreaming)
	{
		GetWorkingSoundWaveData()->bIsStreaming = false;
	}
	else
	{
		GetWorkingSoundWaveData()->bIsStreaming = true;
	}

	return GetWorkingSoundWaveData()->bIsStreaming;
}

bool USoundWave::ShouldUseStreamCaching() const
{
	const bool bIsStreaming = IsStreaming(nullptr);
	GetWorkingSoundWaveData()->bShouldUseStreamCaching = bIsStreaming;

	return bIsStreaming;
}

TArrayView<const uint8> USoundWave::GetZerothChunk(bool bForImmediatePlayback)
{
	if(IsTemplate() || IsRunningDedicatedServer() || !FApp::CanEverRenderAudio())
	{
		return TArrayView<const uint8>();
	}

	// NOT using stream caching
	if (false == ShouldUseStreamCaching())
	{
		FAudioChunkHandle ChunkHandle = IStreamingManager::Get().GetAudioStreamingManager().GetLoadedChunk(GetWorkingSoundWaveData(), 0);
		return TArrayView<const uint8>(ChunkHandle.GetData(), ChunkHandle.Num());
	}

	// using stream caching
	// In editor, we actually don't have a zeroth chunk until we try to play an audio file.
	if (GIsEditor)
	{
		LoadZerothChunk();
	}

	check(GetWorkingSoundWaveData()->GetZerothChunkData().GetView().Num() > 0);

	if (GetNumChunks() > 1)
	{
		// Prime first chunk for playback.
		IStreamingManager::Get().GetAudioStreamingManager().RequestChunk(GetWorkingSoundWaveData(), 1, [](EAudioChunkLoadResult InResult) {}, ENamedThreads::AnyThread, bForImmediatePlayback);
	}

	FBulkDataBuffer<uint8>::ViewType View = GetWorkingSoundWaveData()->GetZerothChunkData().GetView();
	check(View.Num() <= MAX_int32);
	return TArrayView<uint8>(View.GetData(), static_cast<int32>(View.Num()));
}

bool USoundWave::IsSeekable() const
{
	// Note derived forms of SoundWave. (i.e. USoundWaveProcedural and USoundSourceBus)
	// override this call to return false
	return GetWorkingSoundWaveData()->bIsSeekable;
}

bool USoundWave::GetSoundWavesWithCookedAnalysisData(TArray<USoundWave*>& OutSoundWaves)
{
	if (CookedSpectralTimeData.Num() > 0 || CookedEnvelopeTimeData.Num() > 0)
	{
		OutSoundWaves.Add(this);
		return true;
	}
	return false;
}

bool USoundWave::HasCookedFFTData() const
{
	return CookedSpectralTimeData.Num() > 0;
}

bool USoundWave::HasCookedAmplitudeEnvelopeData() const
{
	return CookedEnvelopeTimeData.Num() > 0;
}

int32 USoundWave::GetCompressionQuality() const
{
	if (SoundAssetCompressionType == ESoundAssetCompressionType::PCM)
	{
		// PCM is treated as ADPCM at 100-quality
		return 100;
	}
	else
	{
		return CompressionQuality;
	}
}

TSharedRef<const FSoundWaveProxy> USoundWave::GetSoundWaveProxy()
{
	if (!FApp::CanEverRenderAudio())
	{
		return Proxy;
	}
	
	LLM_SCOPE(ELLMTag::AudioSoundWaveProxies);

#if WITH_EDITORONLY_DATA
	LoadZerothChunk();
#endif // #if WITH_EDITORONLY_DATA

#if UE_USOUNDWAVE_HOT_RELOADING_ENABLED
	return Proxy;
#else // #if UE_USOUNDWAVE_HOT_RELOADING_ENABLED
	return FSoundWaveProxy::CreateNonUpdatingCopy(Proxy);
#endif // #if UE_USOUNDWAVE_HOT_RELOADING_ENABLED
}

FSoundWaveProxyPtr USoundWave::CreateSoundWaveProxy()
{
	return ConstCastSharedPtr<FSoundWaveProxy>(GetSoundWaveProxy().ToSharedPtr());
}

TSharedPtr<Audio::IProxyData> USoundWave::CreateProxyData(const Audio::FProxyDataInitParams& InitParams)
{
	LLM_SCOPE(ELLMTag::AudioSoundWaveProxies);
	
#if UE_USOUNDWAVE_HOT_RELOADING_ENABLED
	return Proxy;
#else // #if UE_USOUNDWAVE_HOT_RELOADING_ENABLED
	return FSoundWaveProxy::CreateNonUpdatingCopy(Proxy).ToSharedPtr();
#endif // #if UE_USOUNDWAVE_HOT_RELOADING_ENABLED
}

void USoundWave::AddPlayingSource(const FSoundWaveClientPtr& Source)
{
	check(Source);
	check(IsInAudioThread() || IsInGameThread());   // Don't allow incrementing on other threads as it's not safe (for GCing of this wave).
	if (Source)
	{
		UE::TUniqueLock Lock(SourcesPlayingCs);
		check(!SourcesPlaying.Contains(Source));
		SourcesPlaying.Add(Source);
	}
}

void USoundWave::RemovePlayingSource(const FSoundWaveClientPtr& Source)
{
	if (Source)
	{
		UE::TUniqueLock Lock(SourcesPlayingCs);
		check(SourcesPlaying.Contains(Source));
		SourcesPlaying.RemoveSwap(Source);
	}
}

void USoundWave::UpdatePlatformData()
{
	if (!FApp::CanEverRenderAudio())
	{
		return;
	}

#if UE_USOUNDWAVE_HOT_RELOADING_ENABLED
	// Publish to the proxy for both streaming and inline waves: FSoundWaveProxy
	// is the public handle on the latest WorkingSoundWaveData, and callers of
	// GetSoundWaveDataRef() expect fresh metadata after a mutation. Gating this
	// on IsStreaming() left inline (ForceInline) waves with a stale, default-
	// constructed FSoundWaveData.
	Proxy->Publish(WorkingSoundWaveData);
#endif // #if UE_USOUNDWAVE_HOT_RELOADING_ENABLED

	if (IsStreaming(nullptr))
	{
#if WITH_EDITORONLY_DATA
		// Recache platform data if the source has changed.
		CachePlatformData(true /* bAsyncCache */);
#endif
	}
}

float USoundWave::GetSampleRateForCurrentPlatform() const
{
	if (bProcedural)
	{
		if (SampleRate > 0)
		{
			return SampleRate;
		}

		// Never return a 0 if we've been serialized with one.
		static const float DefaultSampleRate = 48000.f;
		return DefaultSampleRate;
	}

#if WITH_EDITORONLY_DATA
	// GIsEditor is false in standalone but WITH_EDITORONLY_DATA is true
	if (GIsEditor)
	{
		if (bSampleRateManuallyReset)
		{
			return SampleRate;
		}
		
		float SampleRateOverride = FPlatformCompressionUtilities::GetTargetSampleRateForPlatform(SampleRateQuality);
		if (SampleRateOverride > 0)
		{
			// If we don't have an imported sample rate defined (i.e. old asset), just use the override. This will actually update the imported sample rate when it recooks.
			if (!ImportedSampleRate)
			{
				return SampleRateOverride;
			}
			else
			{
				// Otherwise, use the min of the imported SR and the override
				return FMath::Min(SampleRateOverride, (float)ImportedSampleRate);
			}
		}
		else
		{
			if (!ImportedSampleRate)
			{
				return SampleRate;
			}
			else
			{
				return ImportedSampleRate;
			}
		}
	}
	else
#endif
	{
		if (bCachedSampleRateFromPlatformSettings)
		{
			return CachedSampleRateOverride;
		}
		else if (bSampleRateManuallyReset)
		{
			CachedSampleRateOverride = SampleRate;

			// Only latch the sample rate cached value once we have a valid value
			if (CachedSampleRateOverride > 0)
			{
				bCachedSampleRateFromPlatformSettings = true;
			}

			return CachedSampleRateOverride;
		}
		else
		{
			CachedSampleRateOverride = FPlatformCompressionUtilities::GetTargetSampleRateForPlatform(SampleRateQuality);
			if (CachedSampleRateOverride < 0 || SampleRate < CachedSampleRateOverride)
			{
				CachedSampleRateOverride = SampleRate;
			}

			// Same rationale as the bSampleRateManuallyReset branch — refuse to cache a samplerate zero
			// so an early caller doesn't poison subsequent calls after Serialize.
			if (CachedSampleRateOverride > 0)
			{
				bCachedSampleRateFromPlatformSettings = true;
			}
			return CachedSampleRateOverride;
		}
	}
}

float USoundWave::GetSampleRateForCompressionOverrides(const FPlatformAudioCookOverrides* CompressionOverrides) const
{
	const float* SampleRatePtr = CompressionOverrides->PlatformSampleRates.Find(SampleRateQuality);
	if (SampleRatePtr && *SampleRatePtr > 0.0f)
	{
		// If in editor (but not cooking), then read from current platform 
		// otherwise, use the given cook override sample rate 
		if (!IsRunningCookCommandlet() && GIsEditor)
		{
			return GetSampleRateForCurrentPlatform();
		}
		return FMath::Min(*SampleRatePtr, static_cast<float>(SampleRate));
	}
	else
	{
		return -1.0f;
	}
}

bool USoundWave::GetChunkData(int32 ChunkIndex, uint8** OutChunkData, bool bMakeSureChunkIsLoaded /* = false */)
{
	if (GetWorkingSoundWaveData()->RunningPlatformData.GetChunkFromDDC(ChunkIndex, OutChunkData, bMakeSureChunkIsLoaded) == 0)
	{
#if WITH_EDITORONLY_DATA
		// Unable to load chunks from the cache. Rebuild the sound and attempt to recache it.
		UE_LOGF(LogAudio, Display, "GetChunkData failed, rebuilding %ls", *GetPathName());

		ForceRebuildPlatformData();
		if (GetWorkingSoundWaveData()->RunningPlatformData.GetChunkFromDDC(ChunkIndex, OutChunkData, bMakeSureChunkIsLoaded) == 0)
		{
			UE_LOGF(LogAudio, Warning, "Failed to build sound %ls.", *GetPathName());
		}
		else
		{
			// Succeeded after rebuilding platform data
			return true;
		}
#else
		// Failed to find the SoundWave chunk in the cooked package.
		UE_LOGF(LogAudio, Warning, "GetChunkData failed while streaming. Ensure the following file is cooked: %ls", *GetPathName());
#endif // #if WITH_EDITORONLY_DATA
		return false;
	}
	return true;
}

void USoundWave::OverrideWaveData(int32 InNumChannels, int32 InNumFrames, int32 InSampleRate)
{
#if UE_USOUNDWAVE_HOT_RELOADING_ENABLED
	check(InNumChannels > 0);
	check(InSampleRate > 0);

	if (InNumChannels != GetWorkingSoundWaveData()->NumChannels || InNumFrames != GetWorkingSoundWaveData()->NumFrames || InSampleRate != GetWorkingSoundWaveData()->SampleRate)
	{
		// Replace our SoundWaveData with one with the bare minimum information needed to use it for playback.
		WorkingSoundWaveData = MakeShared<FSoundWaveData>(); // note: at this point OldSoundWaveData is an invalid reference
		GetWorkingSoundWaveData()->NumChannels = InNumChannels;
		GetWorkingSoundWaveData()->NumFrames = InNumFrames;
		GetWorkingSoundWaveData()->SampleRate = InSampleRate;
		GetWorkingSoundWaveData()->Duration = InNumFrames / float(InSampleRate);
		GetWorkingSoundWaveData()->NameCached = GetFName();
		Proxy->Publish(WorkingSoundWaveData);
	}
#else // #if UE_USOUNDWAVE_HOT_RELOADING_ENABLED
	return;
#endif // #if UE_USOUNDWAVE_HOT_RELOADING_ENABLED
}

uint32 USoundWave::GetInterpolatedCookedFFTDataForTimeInternal(float InTime, uint32 StartingIndex, TArray<FSoundWaveSpectralData>& OutData, bool bLoop)
{
	// Find the two entries on either side of the input time
	int32 NumDataEntries = CookedSpectralTimeData.Num();
	for (int32 Index = StartingIndex; Index < NumDataEntries; ++Index)
	{
		// Get the current data at this index
		const FSoundWaveSpectralTimeData& CurrentData = CookedSpectralTimeData[Index];

		// Get the next data, wrap if needed (i.e. if current is last index, we'll lerp to the first index)
		const FSoundWaveSpectralTimeData& NextData = CookedSpectralTimeData[(Index + 1) % NumDataEntries];

		if (InTime >= CurrentData.TimeSec && InTime < NextData.TimeSec)
		{
			// Lerping alpha is fraction from current to next data
			const float Alpha = (InTime - CurrentData.TimeSec) / (NextData.TimeSec - CurrentData.TimeSec);
			for (int32 FrequencyIndex = 0; FrequencyIndex < FrequenciesToAnalyze.Num(); ++FrequencyIndex)
			{
				FSoundWaveSpectralData InterpolatedData;
				InterpolatedData.FrequencyHz = FrequenciesToAnalyze[FrequencyIndex];
				InterpolatedData.Magnitude = FMath::Lerp(CurrentData.Data[FrequencyIndex].Magnitude, NextData.Data[FrequencyIndex].Magnitude, Alpha);
				InterpolatedData.NormalizedMagnitude = FMath::Lerp(CurrentData.Data[FrequencyIndex].NormalizedMagnitude, NextData.Data[FrequencyIndex].NormalizedMagnitude, Alpha);

				OutData.Add(InterpolatedData);
			}

			// Sort by frequency (lowest frequency first).
			OutData.Sort(FCompareSpectralDataByFrequencyHz());

			// We found cooked spectral data which maps to these indices
			return Index;
		}
	}

	return INDEX_NONE;
}

bool USoundWave::GetInterpolatedCookedFFTDataForTime(float InTime, uint32& InOutLastIndex, TArray<FSoundWaveSpectralData>& OutData, bool bLoop)
{
	if (CookedSpectralTimeData.Num() > 0)
	{
		// Handle edge cases
		if (!bLoop)
		{
			// Pointer to which data to use
			FSoundWaveSpectralTimeData* SpectralTimeData = nullptr;

			// We are past the edge
			if (InTime >= CookedSpectralTimeData.Last().TimeSec)
			{
				SpectralTimeData = &CookedSpectralTimeData.Last();
				InOutLastIndex = CookedPlatformData.Num() - 1;
			}
			// We are before the first data point
			else if (InTime < CookedSpectralTimeData[0].TimeSec)
			{
				SpectralTimeData = &CookedSpectralTimeData[0];
				InOutLastIndex = 0;
			}

			// If we were either case before we have a non-nullptr here
			if (SpectralTimeData != nullptr)
			{
				// Create an entry for this clamped output
				for (int32 FrequencyIndex = 0; FrequencyIndex < FrequenciesToAnalyze.Num(); ++FrequencyIndex)
				{
					FSoundWaveSpectralData InterpolatedData;
					InterpolatedData.FrequencyHz = FrequenciesToAnalyze[FrequencyIndex];
					InterpolatedData.Magnitude = SpectralTimeData->Data[FrequencyIndex].Magnitude;
					InterpolatedData.NormalizedMagnitude = SpectralTimeData->Data[FrequencyIndex].NormalizedMagnitude;

					OutData.Add(InterpolatedData);
				}

				return true;
			}
		}
		// We're looping
		else
		{
			// Need to check initial wrap-around case (i.e. we're reading earlier than first data point so need to lerp from last data point to first
			if (InTime >= 0.0f && InTime < CookedSpectralTimeData[0].TimeSec)
			{
				const FSoundWaveSpectralTimeData& CurrentData = CookedSpectralTimeData.Last();

				// Get the next data, wrap if needed (i.e. if current is last index, we'll lerp to the first index)
				const FSoundWaveSpectralTimeData& NextData = CookedSpectralTimeData[0];

				float TimeLeftFromLastDataToEnd = Duration - CurrentData.TimeSec;
				float Alpha = (TimeLeftFromLastDataToEnd + InTime) / (TimeLeftFromLastDataToEnd + NextData.TimeSec);

				for (int32 FrequencyIndex = 0; FrequencyIndex < FrequenciesToAnalyze.Num(); ++FrequencyIndex)
				{
					FSoundWaveSpectralData InterpolatedData;
					InterpolatedData.FrequencyHz = FrequenciesToAnalyze[FrequencyIndex];
					InterpolatedData.Magnitude = FMath::Lerp(CurrentData.Data[FrequencyIndex].Magnitude, NextData.Data[FrequencyIndex].Magnitude, Alpha);
					InterpolatedData.NormalizedMagnitude = FMath::Lerp(CurrentData.Data[FrequencyIndex].NormalizedMagnitude, NextData.Data[FrequencyIndex].NormalizedMagnitude, Alpha);

					OutData.Add(InterpolatedData);

					InOutLastIndex = 0;
				}
				return true;
			}
			// Or we've been offset a bit in the negative.
			else if (InTime < 0.0f)
			{
				// Wrap the time to the end of the sound wave file
				InTime = FMath::Clamp(Duration + InTime, 0.0f, Duration);
			}
		}

		uint32 StartingIndex = InOutLastIndex == INDEX_NONE ? 0 : InOutLastIndex;

		InOutLastIndex = GetInterpolatedCookedFFTDataForTimeInternal(InTime, StartingIndex, OutData, bLoop);
		if (InOutLastIndex == INDEX_NONE && StartingIndex != 0)
		{
			InOutLastIndex = GetInterpolatedCookedFFTDataForTimeInternal(InTime, 0, OutData, bLoop);
		}
		return InOutLastIndex != INDEX_NONE;
	}

	return false;
}

uint32 USoundWave::GetInterpolatedCookedEnvelopeDataForTimeInternal(float InTime, uint32 StartingIndex, float& OutAmplitude, bool bLoop)
{
	if (StartingIndex == INDEX_NONE || StartingIndex == CookedEnvelopeTimeData.Num())
	{
		StartingIndex = 0;
	}

	// Find the two entries on either side of the input time
	int32 NumDataEntries = CookedEnvelopeTimeData.Num();
	for (int32 Index = StartingIndex; Index < NumDataEntries; ++Index)
	{
		const FSoundWaveEnvelopeTimeData& CurrentData = CookedEnvelopeTimeData[Index];
		const FSoundWaveEnvelopeTimeData& NextData = CookedEnvelopeTimeData[(Index + 1) % NumDataEntries];

		if (InTime >= CurrentData.TimeSec && InTime < NextData.TimeSec)
		{
			// Lerping alpha is fraction from current to next data
			const float Alpha = (InTime - CurrentData.TimeSec) / (NextData.TimeSec - CurrentData.TimeSec);
			OutAmplitude = FMath::Lerp(CurrentData.Amplitude, NextData.Amplitude, Alpha);

			// We found cooked spectral data which maps to these indices
			return Index;
		}
	}

	// Did not find the data
	return INDEX_NONE;
}

bool USoundWave::GetInterpolatedCookedEnvelopeDataForTime(float InTime, uint32& InOutLastIndex, float& OutAmplitude, bool bLoop)
{
	InOutLastIndex = INDEX_NONE;
	if (CookedEnvelopeTimeData.Num() > 0 && InTime >= 0.0f)
	{
		// Handle edge cases
		if (!bLoop)
		{
			// We are past the edge
			if (InTime >= CookedEnvelopeTimeData.Last().TimeSec)
			{
				OutAmplitude = CookedEnvelopeTimeData.Last().Amplitude;
				InOutLastIndex = CookedEnvelopeTimeData.Num() - 1;
				return true;
			}
			// We are before the first data point
			else if (InTime < CookedEnvelopeTimeData[0].TimeSec)
			{
				OutAmplitude = CookedEnvelopeTimeData[0].Amplitude;
				InOutLastIndex = 0;
				return true;
			}
		}

		// Need to check initial wrap-around case (i.e. we're reading earlier than first data point so need to lerp from last data point to first
		if (InTime >= 0.0f && InTime < CookedEnvelopeTimeData[0].TimeSec)
		{
			const FSoundWaveEnvelopeTimeData& CurrentData = CookedEnvelopeTimeData.Last();
			const FSoundWaveEnvelopeTimeData& NextData = CookedEnvelopeTimeData[0];

			float TimeLeftFromLastDataToEnd = Duration - CurrentData.TimeSec;
			float Alpha = (TimeLeftFromLastDataToEnd + InTime) / (TimeLeftFromLastDataToEnd + NextData.TimeSec);

			OutAmplitude = FMath::Lerp(CurrentData.Amplitude, NextData.Amplitude, Alpha);
			InOutLastIndex = 0;
			return true;
		}
		// Or we've been offset a bit in the negative.
		else if (InTime < 0.0f)
		{
			// Wrap the time to the end of the sound wave file
			InTime = FMath::Clamp(Duration + InTime, 0.0f, Duration);
		}

		uint32 StartingIndex = InOutLastIndex == INDEX_NONE ? 0 : InOutLastIndex;

		InOutLastIndex = GetInterpolatedCookedEnvelopeDataForTimeInternal(InTime, StartingIndex, OutAmplitude, bLoop);
		if (InOutLastIndex == INDEX_NONE && StartingIndex != 0)
		{
			InOutLastIndex = GetInterpolatedCookedEnvelopeDataForTimeInternal(InTime, 0, OutAmplitude, bLoop);
		}
	}
	return InOutLastIndex != INDEX_NONE;
}

void USoundWave::GetHandleForChunkOfAudio(TFunction<void(FAudioChunkHandle&&)> OnLoadCompleted, bool bForceSync /*= false*/, int32 ChunkIndex /*= 1*/, ENamedThreads::Type CallbackThread /*= ENamedThreads::GameThread*/)
{
	ENamedThreads::Type ThreadToDispatchCallbackOn = (DispatchToGameThreadOnChunkRequestCVar != 0) ? ENamedThreads::GameThread : ENamedThreads::AnyThread;

	// if we are requesting a chunk that is out of bounds,
	// or we shouldn't be interacting w/ the cache, early exit.
	if (!FApp::CanEverRenderAudio() || (ChunkIndex >= static_cast<int32>(GetNumChunks())))
	{
		FAudioChunkHandle EmptyChunkHandle;
		OnLoadCompleted(MoveTemp(EmptyChunkHandle));
	}
	else if (bForceSync)
	{
		// For sync cases, we call GetLoadedChunk with bBlockForLoad = true, then execute the callback immediately.
		FAudioChunkHandle ChunkHandle = IStreamingManager::Get().GetAudioStreamingManager().GetLoadedChunk(GetWorkingSoundWaveData(), ChunkIndex, true);
		OnLoadCompleted(MoveTemp(ChunkHandle));
	}
	else
	{
		TWeakObjectPtr<USoundWave> WeakThis = MakeWeakObjectPtr(this);

		// For async cases, we call RequestChunk and request the loaded chunk in the completion callback
		IStreamingManager::Get().GetAudioStreamingManager().RequestChunk(GetWorkingSoundWaveData(), ChunkIndex, [ThreadToDispatchCallbackOn, WeakThis, OnLoadCompleted, ChunkIndex, CallbackThread](EAudioChunkLoadResult LoadResult)
		{
			auto DispatchOnLoadCompletedCallback = [ThreadToDispatchCallbackOn, OnLoadCompleted, CallbackThread](FAudioChunkHandle&& InHandle)
			{
				if (CallbackThread == ThreadToDispatchCallbackOn)
				{
					OnLoadCompleted(MoveTemp(InHandle));
				}
				else
				{
					// If the callback was requested on a non-game thread, dispatch the callback to that thread.
					AsyncTask(CallbackThread, [OnLoadCompleted, InHandle]() mutable
					{
						OnLoadCompleted(MoveTemp(InHandle));
					});
				}
			};

			// If the USoundWave has been GC'd by the time this chunk finishes loading, abandon ship.
			if (WeakThis.IsValid() && (LoadResult == EAudioChunkLoadResult::Completed || LoadResult == EAudioChunkLoadResult::AlreadyLoaded))
			{
				USoundWave* ThisSoundWave = WeakThis.Get();
				FAudioChunkHandle ChunkHandle = IStreamingManager::Get().GetAudioStreamingManager().GetLoadedChunk(ThisSoundWave->GetWorkingSoundWaveData(), ChunkIndex, (BlockOnChunkLoadCompletionCVar != 0));

				// If we hit this, something went wrong in GetLoadedChunk.
				if (!ChunkHandle.IsValid())
				{
					UE_LOGF(LogAudio, Display, "Failed to retrieve chunk %d from sound %ls after successfully requesting it!", ChunkIndex, *(WeakThis->GetName()));
				}
				DispatchOnLoadCompletedCallback(MoveTemp(ChunkHandle));
			}
			else
			{
				// Load failed. Return an invalid chunk handle.
				FAudioChunkHandle ChunkHandle;
				DispatchOnLoadCompletedCallback(MoveTemp(ChunkHandle));
			}
		}, ThreadToDispatchCallbackOn);
	}
}

void USoundWave::RetainCompressedAudio(bool bForceSync /*= false*/)
{
	
	// Since the zeroth chunk is always inlined and stored in memory,
	// early exit if we only have one chunk.
	if (GIsEditor || IsTemplate() || IsRunningDedicatedServer() || !IsStreaming() || !FApp::CanEverRenderAudio() || DisableRetainingCVar || GetNumChunks() <= 1)
	{
		return;
	}

	// If the first chunk is already loaded and being retained,
	// don't kick off another load.
	if (GetWorkingSoundWaveData()->FirstChunk.IsValid())
	{
		return;
	}
	else if (bForceSync)
	{
		GetWorkingSoundWaveData()->FirstChunk = IStreamingManager::Get().GetAudioStreamingManager().GetLoadedChunk(GetWorkingSoundWaveData(), 1, true);
		UE_CLOGF(Audio::MatchesLogFilter(*GetName()), LogAudioTiming, Verbose, "RetainCompressedAudio (Sync) on SoundWave: Wave='%ls'\tFormat='%ls'\tLooping=%ls\tChunkSize=%u", 
						*GetWorkingSoundWaveData()->GetFName().ToString(),
						*GetWorkingSoundWaveData()->GetRuntimeFormat().ToString(),
						ToCStr(LexToString(GetWorkingSoundWaveData()->IsLooping())),
						GetWorkingSoundWaveData()->FirstChunk.IsValid() ? GetWorkingSoundWaveData()->FirstChunk.Num() : 0
					);

		UE_CLOGF(!(GetWorkingSoundWaveData()->FirstChunk.IsValid()), LogAudio, Display, "First chunk was invalid after synchronous load in RetainCompressedAudio(). This was likely because the cache was blown. Sound: %ls", *GetFullName());
	}
	else
	{
		GetHandleForChunkOfAudio([WeakThis = MakeWeakObjectPtr(this)](FAudioChunkHandle&& OutHandle)
		{
			if (OutHandle.IsValid())
			{
				AsyncTask(ENamedThreads::GameThread, [WeakThis, MovedHandle = MoveTemp(OutHandle)]() mutable
				{
					check(IsInGameThread());

					if (WeakThis.IsValid())
					{
						UE_CLOGF(Audio::MatchesLogFilter(WeakThis->GetWorkingSoundWaveData()->GetFName()), LogAudioTiming, Verbose, "Async SetFirstChunk on SoundWave: Wave='%ls'\tFormat='%ls'\tLooping=%ls\tChunkSize=%u",
							*WeakThis->GetWorkingSoundWaveData()->GetFName().ToString(),
							*WeakThis->GetWorkingSoundWaveData()->GetRuntimeFormat().ToString(),
							ToCStr(LexToString(WeakThis->GetWorkingSoundWaveData()->IsLooping())),
							MovedHandle.Num()
						);
						WeakThis->GetWorkingSoundWaveData()->FirstChunk = MoveTemp(MovedHandle);
					}
				});
				
			}
		}, false, 1);
	}
}

void USoundWave::ReleaseCompressedAudio()
{
	// Here we release the USoundWave's handle to the compressed asset by resetting it.
	GetWorkingSoundWaveData()->FirstChunk = FAudioChunkHandle();
}

bool USoundWave::IsRetainingAudio()
{
	return GetWorkingSoundWaveData()->FirstChunk.IsValid();
}

void USoundWave::OverrideLoadingBehavior(ESoundWaveLoadingBehavior InLoadingBehavior)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(USoundWave::OverrideLoadingBehavior);


	// Don't allow when running the cook commandlet.
	if (IsRunningCookCommandlet())
	{
		return;
	}

	const bool bCanOverrideLoadingBehavior = (LoadingBehavior != ESoundWaveLoadingBehavior::ForceInline);

	if (bCanOverrideLoadingBehavior)
	{
		const ESoundWaveLoadingBehavior OldBehavior = GetLoadingBehavior(false);
		const bool bAlreadyLoaded = !HasAnyFlags(RF_NeedLoad);

		// Already set to a higher setting (lower enum values are more important)
		if (InLoadingBehavior > OldBehavior)
		{
			return;
		}

		// we don't want to retain in editor, we prime instead
		if (GIsEditor && InLoadingBehavior == ESoundWaveLoadingBehavior::RetainOnLoad)
		{
			InLoadingBehavior = ESoundWaveLoadingBehavior::PrimeOnLoad;
		}

		// record the new loading behavior
		// (if this soundwave isn't loaded yet, 
		// GetWorkingSoundWaveData()->LoadingBehavior will take precedence when it does load)
		GetWorkingSoundWaveData()->LoadingBehavior = InLoadingBehavior;
		GetWorkingSoundWaveData()->bLoadingBehaviorOverridden = true;

		// If we're loading for the cook commandlet, we don't have streamed audio chunks to load.
		const bool bHasBuiltStreamedAudio = !GetOutermost()->HasAnyPackageFlags(PKG_ReloadingForCooker) && FApp::CanEverRenderAudio();

		// Manually perform prime/retain on already loaded sound waves
		if (bHasBuiltStreamedAudio && bAlreadyLoaded && IsStreaming())
		{
			if (InLoadingBehavior == ESoundWaveLoadingBehavior::RetainOnLoad)
			{
				ConditionalPostLoad();
				RetainCompressedAudio();
			}
			else if (InLoadingBehavior == ESoundWaveLoadingBehavior::PrimeOnLoad)
			{
				
#if WITH_EDITOR
				// In editor, just make sure that data is available in the local DDC cache for quick access.
				CachePlatformData(true /* bAsyncCache */);
#else
				if (GetNumChunks() > 1 && FApp::CanEverRenderAudio())
				{
					IStreamingManager::Get().GetAudioStreamingManager().RequestChunk(GetWorkingSoundWaveData(), 1, [](EAudioChunkLoadResult) {});
				}
#endif
			}
		}
	}
}

#if WITH_EDITOR
void USoundWave::UpdateAsset(bool bMarkDirty)
{
	// Free playing sounds
	FreeResources();
	
	// Flush audio rendering commands to ensure stopped sounds have finished decoding
	// before invalidating compressed data.
	FlushAudioRenderingCommands(); 

	InvalidateCompressedData();
	UpdatePlatformData();
	
	// Reset any error state we've encountered.
	ResetError();

	if(bMarkDirty)
	{
		MarkPackageDirty();
	}

	// if we are force inline, we need to make sure the shared data is pulled from the DDC
	// before we attempt to use a decoder on the proxy (not using stream caching)
	if (LoadingBehavior == ESoundWaveLoadingBehavior::ForceInline)
	{
		if (GEngine)
		{
			FAudioDevice* LocalAudioDevice = GEngine->GetMainAudioDeviceRaw();
			if (LocalAudioDevice)
			{
				FName RuntimeFormat = GetRuntimeFormat();

				if (LoadingBehavior == ESoundWaveLoadingBehavior::ForceInline && !GetResourceData())
				{
					InitAudioResource(RuntimeFormat);
					check(GetWorkingSoundWaveData()->ResourceSize > 0);
				}
			}
		}
	}
}

const FWaveTransformUObjectConfiguration& USoundWave::GetTransformationChainConfig() const
{
	return TransformationChainConfig;
}

const FWaveTransformUObjectConfiguration& USoundWave::UpdateTransformations()
{
	// All TransformationChainConfig are initialized in this UpdateTransformations() and can be modified by transformations
	TransformationChainConfig = FWaveTransformUObjectConfiguration();

	if (TotalSamples == 0)
	{
		UE_LOG(LogAudio, Warning, TEXT("Found %" SIZE_T_FMT " total samples when updating transformations on soundwave %s"), size_t(TotalSamples), *GetNameSafe(this));
		UE_LOGF(LogAudio, Warning, "Consider reimporting the asset, transformations might not work correctly");
		return TransformationChainConfig;
	}

	// USoundWave::TotalSamples is named incorrectly and actually represents TotalNumFrames (Duration * SampleRate) 
	// Can't use Duration * SampleRate here as Duration is updated as the waveform changes length
	const int32 OriginalNumFrames = FMath::FloorToInt32(TotalSamples);

	TransformationChainConfig.NumChannels = NumChannels;
	TransformationChainConfig.SampleRate = ImportedSampleRate > 0 ? ImportedSampleRate : GetSampleRateForCurrentPlatform();
	TransformationChainConfig.StartTime = 0.f;
	TransformationChainConfig.EndTime = OriginalNumFrames / TransformationChainConfig.SampleRate;
	TransformationChainConfig.WaveCues = CuePoints;
	TransformationChainConfig.bIsPreviewingLoopRegion = false;
	TransformationChainConfig.bCachedSoundWaveLoopState = static_cast<bool>(bLooping);
	TransformationChainConfig.SetOriginalNumFrames(OriginalNumFrames);

	TSet<UClass*> TransformationClassSet;
	for (int32 i = 0; i < Transformations.Num(); ++i)
	{
		TObjectPtr<UWaveformTransformationBase>& Transformation = Transformations[i];
		
		if (Transformation == nullptr)
		{
			continue;
		}

		UClass* TransformationClass = Transformation->GetClass();

		if (TransformationClassSet.Contains(TransformationClass))
		{
			UE_LOGF(LogAudio, Warning, "Can't have more than one %ls on a SoundWave", *TransformationClass->GetName());
			Transformation = nullptr;

			break;
		}

		TransformationClassSet.Add(Transformation->GetClass());
	}

	//Sort by priority to have a proper order of operations and simplify transformation audio processing
	UWaveformTransformationBase::SortTransformationsArray(Transformations);

	for (TObjectPtr<UWaveformTransformationBase>& Transformation : Transformations)
	{
		if (Transformation)
		{
			Transformation->UpdateConfiguration(TransformationChainConfig);
		}
	}

	if (TransformationChainConfig.bIsPreviewingLoopRegion)
	{
		bLooping = 1;
	}
	else
	{
		bLooping = static_cast<uint8>(TransformationChainConfig.bCachedSoundWaveLoopState);
	}

	return TransformationChainConfig;
}

void USoundWave::OverwriteSoundWaveDataWithTransformationData(const TArrayView<TObjectPtr<UWaveformTransformationBase>> InTransformations)
{
	// If InTransformations array is empty, use the current soundwaves transformations
	TArrayView<TObjectPtr<UWaveformTransformationBase>> TransformationsView = InTransformations.IsEmpty() ? 
		TArrayView<TObjectPtr<UWaveformTransformationBase>>(Transformations) : InTransformations;

	for (const TObjectPtr<UWaveformTransformationBase>& Transformation : TransformationsView)
	{
		if (Transformation)
		{
			Transformation->OverwriteSoundWaveData(*this);
		}
	}

	//Overwrite the wave data for cue points and sample loops
	TArray<FWaveCue> WaveCues;
	TArray<FWaveSampleLoop> WaveSampleLoops;

	for (const FSoundWaveCuePoint& CuePoint : CuePoints)
	{
		FWaveCue WaveCue;
		WaveCue.CuePointID = CuePoint.CuePointID;
		WaveCue.Position = CuePoint.FramePosition;
		WaveCue.Label = CuePoint.Label;
		WaveCue.SampleLength = CuePoint.FrameLength;

		WaveCues.Emplace(WaveCue);
		
		if (CuePoint.IsLoopRegion())
		{
			FWaveSampleLoop WaveSampleLoop;
			WaveSampleLoop.LoopID = CuePoint.CuePointID;
			WaveSampleLoop.StartFrame = CuePoint.FramePosition;
			WaveSampleLoop.EndFrame = CuePoint.FramePosition + CuePoint.FrameLength;

			WaveSampleLoops.Emplace(WaveSampleLoop);
		}
	}

	// Some DAWs require the Cues and SampleLoops to be sorted by frame position and not CueID.
	Algo::Sort(WaveCues, [](const FWaveCue& WaveCueA, const FWaveCue& WaveCueB)
		{
			return WaveCueA.Position < WaveCueB.Position;
		});
	Algo::Sort(WaveSampleLoops, [](const FWaveSampleLoop& LoopA, const FWaveSampleLoop& LoopB)
		{
			return LoopA.StartFrame < LoopB.StartFrame;
		});

	TArray<uint8> WaveFileData;
	TFuture<FSharedBuffer> FutureBuffer = RawData.GetPayload();
	TArray<uint8> InWaveFileData;
	InWaveFileData.Append((uint8*)FutureBuffer.Get().GetData(), FutureBuffer.Get().GetSize());

	// CreateCueAndSampleChunks errors out if there are more than 2 channels
	if (NumChannels > 2)
	{
		if (!WaveCues.IsEmpty() && !WaveSampleLoops.IsEmpty())
		{
			UE_LOGF(LogAudio, Warning, "Files with a channel count > 2 are not currently supported so new cue points or loop regions were not written to the RawData in file: %ls", *GetPathName());
		}

		return;
	}

	if (Audio::SoundFileUtils::CreateCueAndSampleChunks(InWaveFileData, WaveFileData, WaveCues, WaveSampleLoops))
	{
		FSharedBuffer Buffer = FSharedBuffer::Clone(WaveFileData.GetData(), WaveFileData.Num());
		RawData.UpdatePayload(Buffer);
	}
}

FWaveformTransformationInfo USoundWave::GetWaveformTransformationInfo() const
{
	FWaveformTransformationInfo TransformationInfo;
	for (TObjectPtr<UWaveformTransformationBase> Transformation : Transformations)
	{
		if (Transformation)
		{
			Transformation->GetTransformationInfo(TransformationInfo);
		}
	}

	return TransformationInfo;
}

#endif // #if WITH_EDITOR

void USoundWave::CacheInheritedLoadingBehavior() const
{
	check(IsInGameThread());


	// Determine this sound wave's loading behavior and cache it.
	if (LoadingBehavior != ESoundWaveLoadingBehavior::Inherited)
	{
		// If this sound wave specifies it's own loading behavior, use that.
		if (GetWorkingSoundWaveData()->LoadingBehavior == ESoundWaveLoadingBehavior::Uninitialized)
		{
			GetWorkingSoundWaveData()->LoadingBehavior = LoadingBehavior;
#if WITH_EDITORONLY_DATA
			GetWorkingSoundWaveData()->SizeOfFirstAudioChunkInSeconds = SizeOfFirstAudioChunkInSeconds;
#endif //WITH_EDITORONLY_DATA
		}
	}
 	else if (GetWorkingSoundWaveData()->bLoadingBehaviorOverridden)
 	{
 		ensureMsgf(GetWorkingSoundWaveData()->LoadingBehavior != ESoundWaveLoadingBehavior::Inherited, TEXT("SoundCue set loading behavior to Inherited on SoudWave: %s"), *GetFullName());
 	}
	else
	{
		// if this is true then the behavior should not be Inherited here
		check(!GetWorkingSoundWaveData()->bLoadingBehaviorOverridden);

		USoundClass* CurrentSoundClass = GetSoundClass();
		ESoundWaveLoadingBehavior SoundClassLoadingBehavior = ESoundWaveLoadingBehavior::Inherited;
		FPerPlatformFloat SoundClassSizeOfFirstAudioChunkInSeconds = 0;

		// Recurse through this sound class's parents until we find an override.
		while (SoundClassLoadingBehavior == ESoundWaveLoadingBehavior::Inherited && CurrentSoundClass != nullptr)
		{
			SoundClassLoadingBehavior = CurrentSoundClass->Properties.LoadingBehavior;
#if WITH_EDITORONLY_DATA
			SoundClassSizeOfFirstAudioChunkInSeconds = CurrentSoundClass->Properties.SizeOfFirstAudioChunkInSeconds;
#endif //WITH_EDITORONLY_DATA			
			CurrentSoundClass = CurrentSoundClass->ParentClass;
		}

		// If we could not find an override in the sound class hierarchy, use the loading behavior defined by our cvar.
		if (SoundClassLoadingBehavior == ESoundWaveLoadingBehavior::Inherited)
		{
			const ESoundWaveLoadingBehavior DefaultLoadingBehavior = SoundWave_Private::GetDefaultLoadingBehaviorCVar();

			// override this loading behavior w/ our default
			SoundClassLoadingBehavior = DefaultLoadingBehavior;
			GetWorkingSoundWaveData()->bLoadingBehaviorOverridden = true;
		}

		GetWorkingSoundWaveData()->LoadingBehavior = SoundClassLoadingBehavior;
#if WITH_EDITORONLY_DATA		
		GetWorkingSoundWaveData()->SizeOfFirstAudioChunkInSeconds = SoundClassSizeOfFirstAudioChunkInSeconds;
#endif //WITH_EDITORONLY_DATA		
	}
}

void USoundWave::SerializeCuePoints(FArchive& Ar, const bool bIsLoadingFromCookedArchive)
{
	
	TArray<FSoundWaveCuePoint> PlatformCuePoints;
#if WITH_EDITORONLY_DATA
	if (Ar.IsCooking())
	{
		switch (CuePointOrigin)
		{
		case ESoundWaveCuePointOrigin::WaveFile:
			PlatformCuePoints = CuePoints;
			break;
		case ESoundWaveCuePointOrigin::MarkerTransformation:
			PlatformCuePoints = GetWaveformTransformationInfo().AllCuePointsRelativeToStartTime;
			break;
		default:
			PlatformCuePoints = CuePoints;
			break;
		}

		float ResampleRatio = -1.0f;
		const ITargetPlatform* CookingTarget = Ar.CookingTarget();
		if (ensure(CookingTarget) && CookingTarget != nullptr)
		{
			if (const FPlatformAudioCookOverrides* Overrides = FPlatformCompressionUtilities::GetCookOverrides(*CookingTarget->IniPlatformName()))
			{
				if (Overrides->bResampleForDevice)
				{
					// at this point, the ImportedSampleRate should be ready to use!
					ensureMsgf(ImportedSampleRate > 0, TEXT("SerializeCuePoints: %s ImportedSampleRate not set: ImportedSampleRate = %d"), *GetName(), ImportedSampleRate);
					const float TargetSampleRate = GetSampleRateForTargetPlatform(CookingTarget);
					ScaleCuePointsForSampleRate(TargetSampleRate, PlatformCuePoints);
				}
			}
		}
	}
	else if (Ar.IsLoading() && !bIsLoadingFromCookedArchive)
	{
		GetWorkingSoundWaveData()->SetAllCuePoints(GetCuePointsScaledForSampleRate(GetSampleRateForCurrentPlatform()));

		return;
	}
#endif

	if (Ar.IsCooking() || bIsLoadingFromCookedArchive)
	{
		Ar << PlatformCuePoints;
	}

	if (bIsLoadingFromCookedArchive)
	{
		GetWorkingSoundWaveData()->SetAllCuePoints(PlatformCuePoints);
	}
}

#if WITH_EDITOR
bool USoundWave::GetRedrawThumbnail() const
{
	return bNeedsThumbnailGeneration != 0;
}

void USoundWave::SetRedrawThumbnail(bool bInRedraw)
{
	bNeedsThumbnailGeneration = static_cast<uint32>(bInRedraw);
}

bool USoundWave::CanVisualizeAsset() const
{
	return true;
}
#endif // WITH_EDITOR

ESoundWaveLoadingBehavior USoundWave::GetLoadingBehavior(bool bCheckSoundClasses /*= true*/) const
{
	checkf(!bCheckSoundClasses || GetWorkingSoundWaveData()->LoadingBehavior != ESoundWaveLoadingBehavior::Uninitialized,
		TEXT("Calling GetLoadingBehavior() is only valid if bCheckSoundClasses is false (which it %s) or CacheInheritedLoadingBehavior has already been called on the game thread. (SoundWave: %s)")
		, bCheckSoundClasses ? TEXT("is not") : TEXT("is"), *GetFullName());


	if (!bCheckSoundClasses)
	{
		if ((LoadingBehavior != ESoundWaveLoadingBehavior::Inherited && !GetWorkingSoundWaveData()->bLoadingBehaviorOverridden))
		{
			// If this sound wave specifies it's own loading behavior, use that.
			return LoadingBehavior;
		}
		else if (GetWorkingSoundWaveData()->bLoadingBehaviorOverridden)
		{
			// If this sound wave has already had it's loading behavior cached from soundclasses or soundcues, use that.
			return GetWorkingSoundWaveData()->LoadingBehavior;
		}
		else
		{
			// Use the cvar.
			const ESoundWaveLoadingBehavior DefaultSoundWaveBehavior = SoundWave_Private::GetDefaultLoadingBehaviorCVar();
			return DefaultSoundWaveBehavior;
		}
	}
	else if (GetWorkingSoundWaveData()->LoadingBehavior == ESoundWaveLoadingBehavior::Uninitialized)
	{
		CacheInheritedLoadingBehavior();
	}

	// We should have resolved by now.
	ensure(GetWorkingSoundWaveData()->LoadingBehavior != ESoundWaveLoadingBehavior::Inherited);
	return GetWorkingSoundWaveData()->LoadingBehavior;
}

FSoundWaveProxy::FSoundWaveProxy() = default;

FSoundWaveProxy::FSoundWaveProxy(USoundWave* InWave)
{
	check(InWave);
	InitializeFromWave(*InWave);
}

void FSoundWaveProxy::InitializeFromWave(USoundWave& InWave)
{
	LLM_SCOPE(ELLMTag::AudioSoundWaveProxies);

	// this should have been allocated by the USoundWave and should always be valid

	bool bIsStreaming = InWave.IsStreaming(nullptr);

	// non-streaming sources need resource data initialized before the FSoundWaveProxy
	// can be used. 
	if (!InWave.InitAudioResource(InWave.GetRuntimeFormat()) && !bIsStreaming)
	{
		UE_LOG(LogAudio, Warning, TEXT("FSoundWaveProxy failed to InitAudioResource: %s, format %s"), *InWave.GetName(), *WriteToString<64>(InWave.GetRuntimeFormat()));
	}

	// We must either be able to stream data, or have the data already.
	
	bool bHasResourceData = InWave.GetResourceSize() > 0;

	check(bIsStreaming || bHasResourceData);
	if (!bIsStreaming && !bHasResourceData)
	{
		UE_LOGF(LogAudio, Warning, "FSoundWaveProxy doesn't have audio data! bIsStreaming = %d, bHasResourceData = %d, Sound = %ls", bIsStreaming, bHasResourceData, *InWave.GetName());
	}
	TypeHash = ::GetTypeHash(&InWave);
}

FSoundWaveProxy::~FSoundWaveProxy()
{
}

TSharedRef<FSoundWaveProxy> FSoundWaveProxy::CreateNonUpdatingCopy(const TSharedRef<const FSoundWaveProxy>& Other)
{
	TSharedRef<FSoundWaveProxy> Result = MakeShared<FSoundWaveProxy>();
	Result->PublishedSoundWaveData = Other->PublishedSoundWaveData;
	
	return Result;
}

TSharedRef<const FSoundWaveProxy> FSoundWaveProxy::CreateNonUpdatingCopy(const TSharedRef<const FSoundWaveData>& Other)
{
	TSharedRef<FSoundWaveProxy> Result = MakeShared<FSoundWaveProxy>();
	Result->PublishedSoundWaveData = ConstCastSharedRef<FSoundWaveData>(Other);
	
	return Result;
}

TSharedRef<const FSoundWaveData> FSoundWaveProxy::GetSoundWaveDataRef() const
{
	// if UE_USOUNDWAVE_HOT_RELOADING_ENABLED is false,
	// all readers have their own copy of the shared ref
	// that we are not updating (no need for lock)
#if UE_USOUNDWAVE_HOT_RELOADING_ENABLED
	FReadScopeLock Lock(PublishedSoundWaveDataLock);
#endif // #if UE_USOUNDWAVE_HOT_RELOADING_ENABLED
	return PublishedSoundWaveData;
}

#if UE_USOUNDWAVE_HOT_RELOADING_ENABLED
bool FSoundWaveProxy::IsLatestData(const FSoundWaveData* InData) const
{
	return InData == &(PublishedSoundWaveData.Get());
}
#endif // #if UE_USOUNDWAVE_HOT_RELOADING_ENABLED

#if UE_USOUNDWAVE_HOT_RELOADING_ENABLED
// note: there are still some things the USoundWave will update
// on existing proxies (i.e. certain flags). Letting these pointers alias 
// after "Publish()" is intentional for now.
void FSoundWaveProxy::Publish(const TSharedRef<FSoundWaveData>& InNewData)
{
	FWriteScopeLock Lock(PublishedSoundWaveDataLock);
	PublishedSoundWaveData = InNewData;
}
#endif // #if UE_USOUNDWAVE_HOT_RELOADING_ENABLED

#if WITH_EDITOR
// (deprecated, 5.8)
int32 FSoundWaveProxy::GetCurrentChunkRevision() const
{
	return PublishedSoundWaveData->GetCurrentChunkRevision();
}
#endif // #if WITH_EDITOR

// (deprecated, 5.8)
FStreamedAudioChunk& FSoundWaveProxy::GetChunk(uint32 ChunkIndex)
{
	return PublishedSoundWaveData->GetChunk(ChunkIndex);
}

// (deprecated, 5.8)
int32 FSoundWaveProxy::GetChunkFromDDC(int32 ChunkIndex, uint8** OutChunkData, bool bMakeSureChunkIsLoaded)
{
	return PublishedSoundWaveData->GetChunkFromDDC(ChunkIndex, OutChunkData, bMakeSureChunkIsLoaded);
}

#if WITH_EDITORONLY_DATA
// (deprecated, 5.8)
FString FSoundWaveProxy::GetDerivedDataKey() const
{
	return PublishedSoundWaveData->GetDerivedDataKey();
}
#endif // #if WITH_EDITORONLY_DATA

// (deprecated, 5.8)
int32 FSoundWaveProxy::GetResourceSize() const
{
	return PublishedSoundWaveData->GetResourceSize();
}

// (deprecated, 5.8)
const uint8* FSoundWaveProxy::GetResourceData() const
{
	return PublishedSoundWaveData->GetResourceData();
}
// (deprecated, 5.8)
void FSoundWaveProxy::ReleaseCompressedAudio()
{
	return PublishedSoundWaveData->ReleaseCompressedAudio();
}

// (deprecated, 5.8)
bool FSoundWaveProxy::LoadZerothChunk()
{
	return PublishedSoundWaveData->LoadZerothChunk();
}

// (deprecated, 5.8)
bool FSoundWaveProxy::GetChunkData(int32 ChunkIndex, uint8** OutChunkData, bool bMakeSureChunkIsLoaded)
{
	return PublishedSoundWaveData->GetChunkData(ChunkIndex, OutChunkData, bMakeSureChunkIsLoaded);
}

// (deprecated, 5.8)
const FName& FSoundWaveProxy::GetFName() const
{
	return PublishedSoundWaveData->GetFName();
}

// (deprecated, 5.8)
const FName& FSoundWaveProxy::GetPackageName() const
{
	return PublishedSoundWaveData->GetPackageName();
}

// (deprecated, 5.8)
const FName& FSoundWaveProxy::GetRuntimeFormat() const
{
	return PublishedSoundWaveData->GetRuntimeFormat();
}

// (deprecated, 5.8)
const FObjectKey& FSoundWaveProxy::GetFObjectKey() const
{
	return PublishedSoundWaveData->GetFObjectKey();
}

// (deprecated, 5.8)
float FSoundWaveProxy::GetDuration() const
{
	return PublishedSoundWaveData->GetDuration();
}

// (deprecated, 5.8)
float FSoundWaveProxy::GetSampleRate() const
{
	return PublishedSoundWaveData->GetSampleRate();
}

// (deprecated, 5.8)
int32 FSoundWaveProxy::GetNumFrames() const
{
	return PublishedSoundWaveData->GetNumFrames();
}

// (deprecated, 5.8)
uint32 FSoundWaveProxy::GetNumChunks() const
{
	return PublishedSoundWaveData->GetNumChunks();
}

// (deprecated, 5.8)
uint32 FSoundWaveProxy::GetNumChannels() const
{
	return PublishedSoundWaveData->GetNumChannels();
}

// (deprecated, 5.8)
uint32 FSoundWaveProxy::GetSizeOfChunk(uint32 ChunkIndex) const
{
	return PublishedSoundWaveData->GetSizeOfChunk(ChunkIndex);
}

// (deprecated, 5.8)
const TArray<FSoundWaveCuePoint>& FSoundWaveProxy::GetCuePoints() const
{
	return PublishedSoundWaveData->GetCuePoints();
}

// (deprecated, 5.8)
const TArray<FSoundWaveCuePoint>& FSoundWaveProxy::GetLoopRegions() const
{
	return PublishedSoundWaveData->GetLoopRegions();
}
// (deprecated, 5.8)
bool FSoundWaveProxy::IsLooping() const
{
	return PublishedSoundWaveData->IsLooping();
}

// (deprecated, 5.8)
bool FSoundWaveProxy::IsTemplate() const
{
	return PublishedSoundWaveData->IsTemplate();
}

// (deprecated, 5.8)
bool FSoundWaveProxy::IsStreaming() const
{
	return PublishedSoundWaveData->IsStreaming();
}

// (deprecated, 5.8)
bool FSoundWaveProxy::IsRetainingAudio() const
{
	return PublishedSoundWaveData->IsRetainingAudio();
}

// (deprecated, 5.8)
bool FSoundWaveProxy::ShouldUseStreamCaching() const
{
	return PublishedSoundWaveData->ShouldUseStreamCaching();
}

// (deprecated, 5.8)
bool FSoundWaveProxy::IsSeekable() const
{
	return PublishedSoundWaveData->IsSeekable();
}

// (deprecated, 5.8)
bool FSoundWaveProxy::IsZerothChunkDataLoaded() const
{
	return PublishedSoundWaveData->IsZerothChunkDataLoaded();
}

// (deprecated, 5.8)
bool FSoundWaveProxy::WasLoadingBehaviorOverridden() const
{
	return PublishedSoundWaveData->WasLoadingBehaviorOverridden();
}

// (deprecated, 5.8)
bool FSoundWaveProxy::HasCompressedData(FName Format, ITargetPlatform* TargetPlatform) const
{
	return PublishedSoundWaveData->HasCompressedData(Format, TargetPlatform);
}

// (deprecated, 5.8)
ESoundWaveLoadingBehavior FSoundWaveProxy::GetLoadingBehavior() const
{
	return PublishedSoundWaveData->GetLoadingBehavior();
}

// (deprecated, 5.8)
const TArrayView<uint8> FSoundWaveProxy::GetZerothChunkDataView() const
{
	return PublishedSoundWaveData->GetZerothChunkDataView();
}

// (deprecated, 5.8)
FByteBulkData* FSoundWaveProxy::GetCompressedData(FName Format, const FPlatformAudioCookOverrides* CompressionOverrides)
{
	return PublishedSoundWaveData->GetCompressedData(Format, CompressionOverrides);
}

// (deprecated, 5.8)
const TArray<FSoundWaveCuePoint> FSoundWaveProxy::GetCuePointsAndLoopRegions() const
{
	
	TArray<FSoundWaveCuePoint> CuePointsAndLoopRegions;
	CuePointsAndLoopRegions.Reserve(PublishedSoundWaveData->GetLoopRegions().Num() + PublishedSoundWaveData->GetCuePoints().Num());

	// Previously loops had two cue points (one marked as a cue point, the other a loop region),
	// now they only have one that is marked as a loop region so we need to append Loop Regions.
	// Append must happen in this order to maintain the label for loops that were imported before the fix because
	// meta sounds will pick only one cue point to render if they have matching frames.
	CuePointsAndLoopRegions.Append(PublishedSoundWaveData->GetLoopRegions());
	CuePointsAndLoopRegions.Append(PublishedSoundWaveData->GetCuePoints());
	
	return CuePointsAndLoopRegions;
}

// (deprecated, 5.8)
TArrayView<const uint8> FSoundWaveProxy::GetZerothChunk(const FSoundWaveProxyPtr& SoundWaveProxy, bool bForImmediatePlayback)
{
	if (ensure(SoundWaveProxy.IsValid()))
	{
		const FSoundWaveData& SoundWaveData = *(SoundWaveProxy->GetSoundWaveDataRef());
		if (GIsEditor)
		{
			SoundWaveData.LoadZerothChunk();
		}

		if (SoundWaveData.IsZerothChunkDataLoaded())
		{
			if (SoundWaveData.ShouldUseStreamCaching())
			{
				if (SoundWaveData.GetNumChunks() > 1)
				{
					// Prime first chunk for playback
					IStreamingManager::Get().GetAudioStreamingManager().RequestChunk(SoundWaveData.AsShared(), 1, [](EAudioChunkLoadResult InResult) {}, ENamedThreads::AnyThread, bForImmediatePlayback);
				}

				return SoundWaveData.GetZerothChunkDataView();
			}
			else
			{
				FAudioChunkHandle ChunkHandle = IStreamingManager::Get().GetAudioStreamingManager().GetLoadedChunk(SoundWaveData.AsShared(), 0);
				return TArrayView<const uint8>(ChunkHandle.GetData(), ChunkHandle.Num());
			}
		}
	}

	return {};
}

void USoundWave::SetError(const TCHAR* InErrorMsg)
{
	GetWorkingSoundWaveData()->SetError(InErrorMsg);
}

void USoundWave::ResetError()
{
	GetWorkingSoundWaveData()->ResetError();
}

bool USoundWave::HasError() const
{
	return GetWorkingSoundWaveData()->HasError();
}

#if WITH_EDITORONLY_DATA
TFuture<FSharedBuffer> USoundWave::FEditorAudioBulkData::GetPayload(const EPayloadFlags Flags) const
{
	if (SoundWave == nullptr)
	{
		// Sanity - should never happen! Just forward the data.
		UE_LOGF(LogAudio, Error, "SoundWave pointer is null in FEditorAudioBulkData!");
		return RawData.GetPayload();
	}

	// Per discussions - while the Payload system was designed to return async stuff, it's practically
	// always in a dedicated task and immediately blocked on, so this isn't as bad as it looks. If we wanted
	// to make this properly async we'd just need to grab a copy of the channel offsets in case the containing
	// sound wave wanders off and then drop all of this is a task.

	// Here's our blockage on a potential rehydration.
	FSharedBuffer BulkDataBuffer;
	if (Flags == EPayloadFlags::ReturnCopyWithTransformationData)
	{
		TObjectPtr<USoundWave> DuplicateSoundWave = DuplicateObject<USoundWave>(SoundWave, SoundWave->GetOuter());

		DuplicateSoundWave->OverwriteSoundWaveDataWithTransformationData();
		BulkDataBuffer = DuplicateSoundWave->RawData.RawData.GetPayload().Get();
	}
	else
	{
		BulkDataBuffer = RawData.GetPayload().Get();
	}

	// We need it mutable. Unless the bulk data system holds a copy this should be a steal.
	FUniqueBuffer MutablePayloadBuffer = BulkDataBuffer.MoveToUnique();
	FMutableMemoryView MutablePayload(MutablePayloadBuffer);

	FWaveModInfo WaveInfo;
	if (SoundWave->ChannelOffsets.Num() > 0)
	{
		TArray<int16> Scratch;
		for (int32 i = 0; i < SoundWave->ChannelOffsets.Num(); ++i)
		{
			// If the channel doesn't exist, skip it.
			if (!SoundWave->ChannelSizes[i])
			{
				continue;
			}

			FMutableMemoryView ChannelPayload = MutablePayload.Mid(SoundWave->ChannelOffsets[i], SoundWave->ChannelSizes[i]);
			if (ChannelPayload.GetSize() != SoundWave->ChannelSizes[i])
			{
				UE_LOGF(LogAudio, Error, "Sound wave payload in asset %ls has inconsitent multi channel size for channel %d: ChannelSizes = %d, Available: %lld. Attempting to continue...",
					*SoundWave->GetName(), i, SoundWave->ChannelSizes[i], ChannelPayload.GetSize());

				// On import we fail here, not sure what to do other than try and continue. Presumably this will die
				// during the derived data build or the wave parse below.
			}

			if (!WaveInfo.ReadWaveInfo((const uint8*)ChannelPayload.GetData(), ChannelPayload.GetSize()))
			{
				UE_LOGF(LogAudio, Error, "Sound wave payload in asset %ls failed to parse channel %d as a wave!", *SoundWave->GetName(), i);
			}
			else if (*WaveInfo.pFormatTag == FWaveModInfo::WAVE_INFO_FORMAT_OODLE_WAVE)
			{
				// Convert UEWavComp data back to a WAV file
				*WaveInfo.pFormatTag = FWaveModInfo::WAVE_INFO_FORMAT_PCM;
				int64 PayloadSampleCount = WaveInfo.GetNumSamples();
				int64 PayloadChannelCount = *WaveInfo.pChannels;
				Scratch.SetNumUninitialized(PayloadSampleCount);
				uewav_decode16((int16*)WaveInfo.SampleDataStart, Scratch.GetData(), PayloadSampleCount, PayloadChannelCount);
			}
			// else - it wasn't encoded as uewav so leave it alone.
		}
	}
	else
	{
		if (!WaveInfo.ReadWaveInfo((const uint8*)MutablePayload.GetData(), MutablePayload.GetSize())) 
		{
			UE_LOGF(LogAudio, Error, "Sound wave payload in asset %ls failed to parse as a wave!", *SoundWave->GetName());
		}
		else if (*WaveInfo.pFormatTag == FWaveModInfo::WAVE_INFO_FORMAT_OODLE_WAVE)
		{
			// Convert UEWavComp data back to a WAV file
			*WaveInfo.pFormatTag = FWaveModInfo::WAVE_INFO_FORMAT_PCM;
			int64 PayloadSampleCount = WaveInfo.GetNumSamples();
			int64 PayloadChannelCount = *WaveInfo.pChannels;
			TArray<int16> Scratch;
			Scratch.AddUninitialized(PayloadSampleCount);
			uewav_decode16((int16*)WaveInfo.SampleDataStart, Scratch.GetData(), PayloadSampleCount, PayloadChannelCount);
		}
		// else - not uewav encoded so leave it alone.
	}

	TPromise<FSharedBuffer> promise;
	promise.EmplaceValue(MutablePayloadBuffer.MoveToShared());
	return promise.GetFuture();
}

void USoundWave::FEditorAudioBulkData::UpdatePayload(FSharedBuffer InPayload, UObject* /*Owner - see comments in header declaration*/)
{
	if (SoundWave == nullptr)
	{
		// Sanity - should never happen! Just forward the data.
		UE_LOGF(LogAudio, Error, "SoundWave pointer is null in UpdatePayload!");
		return RawData.UpdatePayload(InPayload, nullptr);
	}

	bool bEnableUEWavComp = false;
	GConfig->GetBool(TEXT("AudioImporter"), TEXT("EnableUEWavComp"), bEnableUEWavComp, GEditorIni);
	if (!bEnableUEWavComp)
	{
		// Passthru if we are disabled.
		return RawData.UpdatePayload(InPayload, SoundWave);
	}

	// We modify in place so we have to move (or hopefully steal) to a mutable copy.
	FUniqueBuffer Buffer = InPayload.MoveToUnique();
	FMutableMemoryView MutablePayload(Buffer);

	if (SoundWave->ChannelOffsets.Num() > 0) 
	{
		// Multichannel - N mono RIFF files packed back to back.
		TArray<int16> Scratch;
		for (int32 i = 0; i < SoundWave->ChannelOffsets.Num(); ++i)
		{
			// If the channel doesn't exist, skip it.
			if (!SoundWave->ChannelSizes[i])
			{
				continue;
			}

			FMutableMemoryView ChannelPayload = MutablePayload.Mid(SoundWave->ChannelOffsets[i], SoundWave->ChannelSizes[i]);
			if (ChannelPayload.GetSize() != SoundWave->ChannelSizes[i])
			{
				UE_LOGF(LogAudio, Error, "New sound wave payload in asset %ls has inconsistent multi channel size for channel %d: ChannelSizes = %d, Available: %lld, payload not updated.",
					*SoundWave->GetName(), i, SoundWave->ChannelSizes[i], ChannelPayload.GetSize());	
				return;
			}

			FWaveModInfo WaveInfo;
			if (!WaveInfo.ReadWaveInfo((const uint8*)ChannelPayload.GetData(), ChannelPayload.GetSize()))
			{
				UE_LOGF(LogAudio, Error, "New sound wave payload failed to parse channel %d as a wave in asset %ls! Payload not updated!", i, *SoundWave->GetFullName());
				return;
			}

			if (*WaveInfo.pFormatTag == FWaveModInfo::WAVE_INFO_FORMAT_PCM)
			{
				int64 PayloadSampleCount = WaveInfo.GetNumSamples();
				int64 PayloadChannelCount = *WaveInfo.pChannels;
				Scratch.SetNumUninitialized(PayloadSampleCount);

				// Samples is modified in place.
				uewav_encode16((int16*)WaveInfo.SampleDataStart, Scratch.GetData(), PayloadSampleCount, PayloadChannelCount);

				// Mark in the wave format that we are OODLE so we know to decode.
				*WaveInfo.pFormatTag = FWaveModInfo::WAVE_INFO_FORMAT_OODLE_WAVE;
			}
			else
			{
				UE_LOGF(LogAudio, Error, "New sound wave payload isn't PCM format for channel %d in asset %ls (reported = %d) - payload not updated!", i, *SoundWave->GetFullName(), *WaveInfo.pFormatTag);
				return;
			}
		}
	} 
	else
	{
		FWaveModInfo WaveInfo;
		if (!WaveInfo.ReadWaveInfo((const uint8*)MutablePayload.GetData(), MutablePayload.GetSize(), 0)) 
		{
			UE_LOGF(LogAudio, Error, "USoundWave new payload failed to parse as a wave in asset %ls! Payload not updated!", *SoundWave->GetName());
			return;
		}		
		
		if (*WaveInfo.pFormatTag == FWaveModInfo::WAVE_INFO_FORMAT_PCM)
		{
			int64 PayloadSampleCount = WaveInfo.GetNumSamples();
			int64 PayloadChannelCount = *WaveInfo.pChannels;
			TArray<int16> Scratch;
			Scratch.AddUninitialized(PayloadSampleCount);

			// Samples is modified in place.
			uewav_encode16((int16*)WaveInfo.SampleDataStart, Scratch.GetData(), PayloadSampleCount, PayloadChannelCount);
				
			// Mark in the wave format that we are OODLE so we know to decode.
			*WaveInfo.pFormatTag = FWaveModInfo::WAVE_INFO_FORMAT_OODLE_WAVE;
		}
		else
		{
			UE_LOGF(LogAudio, Error, "New sound wave payload isn't PCM format in asset %ls (reported = %d) - payload not updated!", *SoundWave->GetFullName(), *WaveInfo.pFormatTag);
			return;
		}
	}

	RawData.UpdatePayload(Buffer.MoveToShared(), SoundWave);
}

bool USoundWave::FEditorAudioBulkData::HasPayloadData() const
{
	return RawData.HasPayloadData();
}

void USoundWave::FEditorAudioBulkData::Serialize(FArchive& Ar, UObject* /*Owner - see comments in header re: SoundWave */, bool bAllowRegister)
{
	RawData.Serialize(Ar, SoundWave, bAllowRegister);
}

void USoundWave::FEditorAudioBulkData::CreateFromBulkData(FBulkData& InBulkData, const FGuid& InGuid, UObject* /*Owner - see comments in header re: SoundWave */)
{
	RawData.CreateFromBulkData(InBulkData, InGuid, SoundWave);
}
#endif

