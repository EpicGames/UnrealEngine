// Copyright Epic Games, Inc. All Rights Reserved.

#include "SoundWaveCompiler.h"
#include "Audio.h"
#include "Misc/QueuedThreadPool.h"
#include "UObject/Package.h"

#if WITH_EDITOR

#include "AsyncCompilationHelpers.h"
#include "AssetCompilingManager.h"
#include "ObjectCacheContext.h"
#include "Settings/EditorExperimentalSettings.h"
#include "Misc/QueuedThreadPoolWrapper.h"
#include "UObject/StrongObjectPtr.h"
#include "ProfilingDebugging/CountersTrace.h"
#include "Sound/SoundWave.h"

#define LOCTEXT_NAMESPACE "SoundWaveCompiler"

static AsyncCompilationHelpers::FAsyncCompilationStandardCVars CVarAsyncSoundWaveStandard(
	TEXT("SoundWave"),
	TEXT("soundwaves"),
	FConsoleCommandDelegate::CreateLambda(
		[]()
		{
			FSoundWaveCompilingManager::Get().FinishAllCompilation();
		}
	));

namespace SoundWaveCompilingManagerImpl
{
	static EQueuedWorkPriority GetBasePriority(USoundWave* InSoundWave)
	{
		return EQueuedWorkPriority::Lowest;
	}

	static EQueuedWorkPriority GetBoostPriority(USoundWave* InSoundWave)
	{
		return (EQueuedWorkPriority)(FMath::Max((uint8)EQueuedWorkPriority::Highest, (uint8)GetBasePriority(InSoundWave)) - 1);
	}

	static void EnsureInitializedCVars()
	{
		static bool bIsInitialized = false;

		if (!bIsInitialized)
		{
			bIsInitialized = true;

			AsyncCompilationHelpers::EnsureInitializedCVars(
				TEXT("SoundWave"),
				CVarAsyncSoundWaveStandard.AsyncCompilation,
				CVarAsyncSoundWaveStandard.AsyncCompilationMaxConcurrency,
				GET_MEMBER_NAME_CHECKED(UEditorExperimentalSettings, bEnableAsyncSoundWaveCompilation));
		}
	}
}

FSoundWaveCompilingManager::FSoundWaveCompilingManager()
	: Notification(MakeUnique<FAsyncCompilationNotification>(GetAssetNameFormat()))
{
	SoundWaveCompilingManagerImpl::EnsureInitializedCVars();
}

FName FSoundWaveCompilingManager::GetStaticAssetTypeName()
{
	return TEXT("UE-SoundWave");
}

FName FSoundWaveCompilingManager::GetAssetTypeName() const
{
	return GetStaticAssetTypeName();
}

TArrayView<FName> FSoundWaveCompilingManager::GetDependentTypeNames() const
{
	return TArrayView<FName>{ };
}

FTextFormat FSoundWaveCompilingManager::GetAssetNameFormat() const
{
	return LOCTEXT("SoundWaveNameFormat", "{0}|plural(one=SoundWave,other=SoundWaves)");
}

EQueuedWorkPriority FSoundWaveCompilingManager::GetBasePriority(USoundWave* InSoundWave) const
{
	return SoundWaveCompilingManagerImpl::GetBasePriority(InSoundWave);
}

FQueuedThreadPool* FSoundWaveCompilingManager::GetThreadPool() const
{
	static FQueuedThreadPoolWrapper* GSoundWaveThreadPool = nullptr;
	if (GSoundWaveThreadPool == nullptr && FAssetCompilingManager::Get().GetThreadPool() != nullptr)
	{
		const auto SoundWavePriorityMapper = [](EQueuedWorkPriority SoundWavePriority) { return FMath::Max(SoundWavePriority, EQueuedWorkPriority::Low); };

		// SoundWaves will be scheduled on the asset thread pool, where concurrency limits might by dynamically adjusted depending on memory constraints.
		GSoundWaveThreadPool = new FQueuedThreadPoolWrapper(FAssetCompilingManager::Get().GetThreadPool(), -1, SoundWavePriorityMapper);

		AsyncCompilationHelpers::BindThreadPoolToCVar(
			GSoundWaveThreadPool,
			CVarAsyncSoundWaveStandard.AsyncCompilation,
			CVarAsyncSoundWaveStandard.AsyncCompilationResume,
			CVarAsyncSoundWaveStandard.AsyncCompilationMaxConcurrency
		);
	}

	return GSoundWaveThreadPool;
}

void FSoundWaveCompilingManager::Shutdown()
{
	bHasShutdown = true;
	if (GetNumRemainingSoundWaves())
	{
		// Wait on SoundWaves already in progress we couldn't cancel
		FinishCompilation(GatherPendingSoundWaves());
	}
}

bool FSoundWaveCompilingManager::IsAsyncSoundWaveCompilationEnabled() const
{
	if (bHasShutdown || !FPlatformProcess::SupportsMultithreading())
	{
		return false;
	}

	return CVarAsyncSoundWaveStandard.AsyncCompilation.GetValueOnAnyThread() != 0;
}

TRACE_DECLARE_INT_COUNTER(QueuedSoundWaveCompilation, TEXT("AsyncCompilation/QueuedSoundWave"));
void FSoundWaveCompilingManager::UpdateCompilationNotification()
{
	TRACE_COUNTER_SET(QueuedSoundWaveCompilation, GetNumRemainingSoundWaves());
	Notification->Update(GetNumRemainingSoundWaves());
}

void FSoundWaveCompilingManager::PostCompilation(USoundWave* SoundWave)
{
	check(IsInGameThread());
	TRACE_CPUPROFILER_EVENT_SCOPE(FSoundWaveCompilingManager::PostCompilation);

	SoundWave->FinishCachePlatformData();
}

bool FSoundWaveCompilingManager::IsAsyncCompilationAllowed(USoundWave* SoundWave) const
{
	return IsAsyncSoundWaveCompilationEnabled();
}

FSoundWaveCompilingManager& FSoundWaveCompilingManager::Get()
{
	static FSoundWaveCompilingManager Singleton;
	return Singleton;
}

int32 FSoundWaveCompilingManager::GetNumRemainingSoundWaves() const
{
	FReadScopeLock ScopeLock(Lock);
	return RegisteredSoundWaves.Num();
}

int32 FSoundWaveCompilingManager::GetNumRemainingAssets() const
{
	return GetNumRemainingSoundWaves();
}

void FSoundWaveCompilingManager::AddSoundWaves(TArrayView<USoundWave* const> InSoundWaves)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FSoundWaveCompilingManager::AddSoundWaves)

	{
		FWriteScopeLock ScopeLock(Lock);
		for (USoundWave* SoundWave : InSoundWaves)
		{
			RegisteredSoundWaves.Emplace(SoundWave);
		}
	}

	TRACE_COUNTER_SET(QueuedSoundWaveCompilation, GetNumRemainingSoundWaves());
}

void FSoundWaveCompilingManager::FinishCompilation(TArrayView<USoundWave* const> InSoundWaves)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FSoundWaveCompilingManager::FinishCompilation);

	using namespace SoundWaveCompilingManagerImpl;
	check(IsInGameThread());

	TSet<USoundWave*> PendingSoundWaves;
	PendingSoundWaves.Reserve(InSoundWaves.Num());

	{
		FReadScopeLock ScopeLock(Lock);
		for (USoundWave* SoundWave : InSoundWaves)
		{
			if (RegisteredSoundWaves.Contains(SoundWave))
			{
				PendingSoundWaves.Add(SoundWave);
			}
		}
	}

	if (PendingSoundWaves.Num())
	{
		class FCompilableSoundWave final : public AsyncCompilationHelpers::ICompilable
		{
		public:
			FCompilableSoundWave(USoundWave* InSoundWave)
				: SoundWave(InSoundWave)
			{
			}

			void Reschedule(FQueuedThreadPool* InThreadPool, EQueuedWorkPriority InPriority) override
			{
				SoundWave->RescheduleAsyncTask(InThreadPool, InPriority);
			}

			bool WaitCompletionWithTimeout(float TimeLimitSeconds) override
			{
				return SoundWave->WaitAsyncTaskWithTimeout(TimeLimitSeconds);
			}

			FName GetName() override { return SoundWave->GetOutermost()->GetFName(); }

			TStrongObjectPtr<USoundWave> SoundWave;
		};

		TArray<USoundWave*> UniqueSoundWaves(PendingSoundWaves.Array());
		TArray<FCompilableSoundWave> CompilableSoundWaves(UniqueSoundWaves);
		using namespace AsyncCompilationHelpers;
		FObjectCacheContextScope ObjectCacheScope;
		AsyncCompilationHelpers::FinishCompilation(
			[&CompilableSoundWaves](int32 Index) -> ICompilable& { return CompilableSoundWaves[Index]; },
			CompilableSoundWaves.Num(),
			LOCTEXT("SoundWaves", "SoundWaves"),
			LogAudio,
			[this](ICompilable* Object)
			{
				USoundWave* SoundWave = static_cast<FCompilableSoundWave*>(Object)->SoundWave.Get();
				PostCompilation(SoundWave);

				{
					FWriteScopeLock ScopeLock(Lock);
					RegisteredSoundWaves.Remove(SoundWave);
				}
			}
		);

		PostCompilation(UniqueSoundWaves);
	}
}

void FSoundWaveCompilingManager::PostCompilation(TArrayView<USoundWave* const> InCompiledSoundWaves)
{
	using namespace SoundWaveCompilingManagerImpl;
	if (InCompiledSoundWaves.Num())
	{
		// Calling this delegate during app exit might be quite dangerous and lead to crash
		// Triggering this callback while garbage collecting can also result in listeners trying to look up objects
		if (!GExitPurge && !IsGarbageCollecting())
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(OnAssetPostCompileEvent);

			TArray<FAssetCompileData> AssetsData;
			AssetsData.Reserve(InCompiledSoundWaves.Num());

			for (USoundWave* SoundWave : InCompiledSoundWaves)
			{
				AssetsData.Emplace(SoundWave);
			}

			FAssetCompilingManager::Get().OnAssetPostCompileEvent().Broadcast(AssetsData);
		}
	}
}

TArray<USoundWave*> FSoundWaveCompilingManager::GatherPendingSoundWaves()
{
	TArray<USoundWave*> PendingSoundWaves;
	PendingSoundWaves.Reserve(GetNumRemainingSoundWaves());

	// Resolve weak pointers and get rid of stale objects at the same time
	FWriteScopeLock ScopeLock(Lock);
	for (auto Iterator = RegisteredSoundWaves.CreateIterator(); Iterator; ++Iterator)
	{
		if (USoundWave* SoundWave = Iterator->Get())
		{
			PendingSoundWaves.Add(SoundWave);
		}
		else
		{
			Iterator.RemoveCurrent();
		}
	}

	return MoveTemp(PendingSoundWaves);
}

void FSoundWaveCompilingManager::FinishAllCompilation()
{
	check(IsInGameThread());
	TRACE_CPUPROFILER_EVENT_SCOPE(FSoundWaveCompilingManager::FinishAllCompilation)

	if (GetNumRemainingSoundWaves())
	{
		FinishCompilation(GatherPendingSoundWaves());
	}
}

void FSoundWaveCompilingManager::FinishCompilationForObjects(TArrayView<UObject* const> InObjects)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FSoundWaveCompilingManager::FinishCompilationForObjects);

	TSet<USoundWave*> SoundWaves;
	for (UObject* Object : InObjects)
	{
		if (USoundWave* SoundWave = Cast<USoundWave>(Object))
		{
			SoundWaves.Add(SoundWave);
		}
	}

	if (SoundWaves.Num())
	{
		FinishCompilation(SoundWaves.Array());
	}
}

void FSoundWaveCompilingManager::ProcessSoundWaves(bool bLimitExecutionTime, int32 MaximumPriority)
{
	using namespace SoundWaveCompilingManagerImpl;
	TRACE_CPUPROFILER_EVENT_SCOPE(FSoundWaveCompilingManager::ProcessSoundWaves);
	const double MaxSecondsPerFrame = 0.016;

	if (GetNumRemainingAssets())
	{
		TArray<USoundWave*> ProcessedSoundWaves;
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(ProcessFinishedSoundWaves);

			TArray<USoundWave*> SoundWavesToProcess = GatherPendingSoundWaves();

			double TickStartTime = FPlatformTime::Seconds();
			for (USoundWave* SoundWave : SoundWavesToProcess)
			{
				const bool bHasTimeLeft = bLimitExecutionTime ? ((FPlatformTime::Seconds() - TickStartTime) < MaxSecondsPerFrame) : true;
				if (bHasTimeLeft && SoundWave->IsAsyncWorkComplete())
				{
					PostCompilation(SoundWave);
					ProcessedSoundWaves.Add(SoundWave);
				}
			}
		}

		if (ProcessedSoundWaves.Num())
		{
			PostCompilation(ProcessedSoundWaves);

			FWriteScopeLock WriteLock(Lock);
			for (USoundWave* ProcessedSoundWave : ProcessedSoundWaves)
			{
				RegisteredSoundWaves.Remove(ProcessedSoundWave);
			}
		}
	}
}

void FSoundWaveCompilingManager::ProcessAsyncTasks(bool bLimitExecutionTime)
{
	FObjectCacheContextScope ObjectCacheScope;

	ProcessSoundWaves(bLimitExecutionTime);

	UpdateCompilationNotification();
}

#undef LOCTEXT_NAMESPACE

#endif // #if WITH_EDITOR
