// Copyright Epic Games, Inc. All Rights Reserved.

#include "Sound/SoundWaveProcedural.h"

#include "AudioDevice.h"
#include "Engine/Engine.h"
#include "UObject/AssetRegistryTagsContext.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SoundWaveProcedural)


USoundWaveProcedural::USoundWaveProcedural(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bProcedural = true;
	bReset = false;
	NumBufferUnderrunSamples = 512;
	NumSamplesToGeneratePerCallback = DEFAULT_PROCEDURAL_SOUNDWAVE_BUFFER_SIZE;
	static_assert(DEFAULT_PROCEDURAL_SOUNDWAVE_BUFFER_SIZE >= 512, "Should generate more samples than this per callback.");
	//checkf(NumSamplesToGeneratePerCallback >= NumBufferUnderrunSamples, TEXT("Should generate more samples than this per callback."));

	// If the main audio device has been set up, we can use this to define our callback size.
	// We need to do this for procedural sound waves that we do not process asynchronously,
	// to ensure that we do not underrun.
	
	SampleByteSize = 2;

}

void USoundWaveProcedural::QueueAudio(const uint8* AudioData, const int32 BufferSize)
{
	Audio::EAudioMixerStreamDataFormat::Type Format = GetGeneratedPCMDataFormat();
	SampleByteSize = (Format == Audio::EAudioMixerStreamDataFormat::Int16) ? 2 : 4;

	if (BufferSize == 0 || !ensure((BufferSize % SampleByteSize) == 0))
	{
		return;
	}

	TArray<uint8> NewAudioBuffer;
	NewAudioBuffer.AddUninitialized(BufferSize);
	FMemory::Memcpy(NewAudioBuffer.GetData(), AudioData, BufferSize);
	QueuedAudio.Enqueue(MoveTemp(NewAudioBuffer));

	AvailableByteCount.Add(BufferSize);
}

void USoundWaveProcedural::PumpQueuedAudio()
{
	// Pump the enqueued audio
	TArray<uint8> NewQueuedBuffer;
	while (QueuedAudio.Dequeue(NewQueuedBuffer))
	{
		AudioBuffer.Append(NewQueuedBuffer);
	}
}

int32 USoundWaveProcedural::GeneratePCMData(uint8* PCMData, const int32 SamplesNeeded)
{
	// Check if we've been told to reset our audio buffer
	if (bReset)
	{
		bReset = false;
		AudioBuffer.Reset();
		AvailableByteCount.Reset();
	}

	Audio::EAudioMixerStreamDataFormat::Type Format = GetGeneratedPCMDataFormat();
	SampleByteSize = (Format == Audio::EAudioMixerStreamDataFormat::Int16) ? 2 : 4;

	int32 SamplesAvailable = AudioBuffer.Num() / SampleByteSize;
	int32 SamplesToGenerate = FMath::Min(NumSamplesToGeneratePerCallback, SamplesNeeded);

	check(SamplesToGenerate >= NumBufferUnderrunSamples);

	bool bPumpQueuedAudio = true;

	if (SamplesAvailable < SamplesToGenerate)
	{
		// First try to use the virtual function which assumes we're writing directly into our audio buffer
		// since we're calling from the audio render thread.
		int32 NumSamplesGenerated = OnGeneratePCMAudio(AudioBuffer, SamplesToGenerate);
		if (NumSamplesGenerated > 0)
		{
			// Shrink the audio buffer size to the actual number of samples generated
			const int32 BytesGenerated = NumSamplesGenerated * SampleByteSize;
			ensureAlwaysMsgf(BytesGenerated <= AudioBuffer.Num(), TEXT("Soundwave Procedural generated more bytes than expected (%d generated, %d expected)"), BytesGenerated, AudioBuffer.Num());
			if (BytesGenerated < AudioBuffer.Num())
			{
				AudioBuffer.SetNum(BytesGenerated, EAllowShrinking::No);
			}
			bPumpQueuedAudio = false;
		}
		else if (OnSoundWaveProceduralUnderflow.IsBound())
		{
			// Note that this delegate may or may not fire inline here. If you need
			// To gaurantee that the audio will be filled, don't use this delegate function
			OnSoundWaveProceduralUnderflow.Execute(this, SamplesToGenerate);
		}
	}

	if (bPumpQueuedAudio)
	{
		PumpQueuedAudio();
	}

	SamplesAvailable = AudioBuffer.Num() / SampleByteSize;

	// Wait until we have enough samples that are requested before starting.
	if (SamplesAvailable > 0)
	{
		const int32 SamplesToCopy = FMath::Min<int32>(SamplesToGenerate, SamplesAvailable);
		const int32 BytesToCopy = SamplesToCopy * SampleByteSize;

		FMemory::Memcpy((void*)PCMData, &AudioBuffer[0], BytesToCopy);
		AudioBuffer.RemoveAt(0, BytesToCopy, EAllowShrinking::No);

		// Decrease the available by count
		if (bPumpQueuedAudio)
		{
			AvailableByteCount.Subtract(BytesToCopy);
		}

		return BytesToCopy;
	}

	// There wasn't enough data ready, write out zeros
	const int32 BytesCopied = NumBufferUnderrunSamples * SampleByteSize;
	FMemory::Memzero(PCMData, BytesCopied);
	return BytesCopied;
}

void USoundWaveProcedural::ResetAudio()
{
	// Empty out any enqueued audio buffers
	QueuedAudio.Empty();

	// Flag that we need to reset our audio buffer (on the audio thread)
	bReset = true;
}

int32 USoundWaveProcedural::GetAvailableAudioByteCount()
{
	return AvailableByteCount.GetValue();
}

int32 USoundWaveProcedural::GetResourceSizeForFormat(FName Format)
{
	return 0;
}

void USoundWaveProcedural::GetAssetRegistryTags(TArray<FAssetRegistryTag>& OutTags) const
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS;
	Super::GetAssetRegistryTags(OutTags);
	PRAGMA_ENABLE_DEPRECATION_WARNINGS;
}

void USoundWaveProcedural::GetAssetRegistryTags(FAssetRegistryTagsContext Context) const
{
	Super::GetAssetRegistryTags(Context);
}

bool USoundWaveProcedural::HasCompressedData(FName Format, ITargetPlatform* TargetPlatform) const
{
	return false;
}

void USoundWaveProcedural::BeginGetCompressedData(FName Format, const FPlatformAudioCookOverrides* CompressionOverrides, const ITargetPlatform* InTargetPlatform)
{
	// SoundWaveProcedural does not have compressed data and should generally not be asked about it
}

FByteBulkData* USoundWaveProcedural::GetCompressedData(FName Format, const FPlatformAudioCookOverrides* CompressionOverrides, const ITargetPlatform* InTargetPlatform )
{
	// SoundWaveProcedural does not have compressed data and should generally not be asked about it
	return nullptr;
}

void USoundWaveProcedural::Serialize(FArchive& Ar)
{
	// Do not call the USoundWave version of serialize
	USoundBase::Serialize(Ar);

#if WITH_EDITORONLY_DATA
	// Due to "skipping" USoundWave::Serialize above, modulation
	// versioning is required to be called explicitly here.
	if (Ar.IsLoading())
	{
		ModulationSettings.VersionModulators();
	}
#endif // WITH_EDITORONLY_DATA
}

void USoundWaveProcedural::InitAudioResource(FByteBulkData& CompressedData)
{
	// Should never be pushing compressed data to a SoundWaveProcedural
	check(false);
}

bool USoundWaveProcedural::InitAudioResource(FName Format)
{
	// Nothing to be done to initialize a USoundWaveProcedural
	return true;
}

