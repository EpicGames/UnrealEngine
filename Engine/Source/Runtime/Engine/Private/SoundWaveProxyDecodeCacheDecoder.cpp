// Copyright Epic Games, Inc. All Rights Reserved.

#include "SoundWaveProxyDecodeCacheDecoder.h"
#include "SoundWaveProxyDecodeCacheImpl.h"

FSoundWaveProxyDecodeCacheDecoder::FSoundWaveProxyDecodeCacheDecoder(const FSoundWaveDataRef& InWaveData)
{
	FName Format = InWaveData->GetRuntimeFormat();
	CompressedAudioInfo = TUniquePtr<ICompressedAudioInfo>(IAudioInfoFactoryRegistry::Get().Create(Format, InWaveData->GetFName()));
	if (!CompressedAudioInfo)
	{
		UE_LOGF(LogAudio, Error, "FSoundWaveProxyDecodeCacheDecoder::FSoundWaveProxyDecodeCacheDecoder() Failed to create a compressed audio info struct for format %ls", *Format.ToString());
	}
	// Need to do this to make sure the wrapper of concrete implementations functions correctly
	NumChannels = InWaveData->GetNumChannels();
	NumFramesInWave = InWaveData->GetNumFrames();
	WaveGuid = InWaveData->GetGUID();
	WaveName = InWaveData->GetFName();
	SampleRate = InWaveData->GetSampleRate();
#if WITH_EDITOR
	ChunkRevision = InWaveData->GetCurrentChunkRevision();
#endif // #if WITH_EDITOR
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
FSoundWaveProxyDecodeCacheDecoder::FSoundWaveProxyDecodeCacheDecoder(const FSoundWaveProxyRef& InWaveProxy)
	: FSoundWaveProxyDecodeCacheDecoder(InWaveProxy->GetSoundWaveDataRef())
{
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

bool FSoundWaveProxyDecodeCacheDecoder::ReadCompressedInfo(const uint8* InSrcBufferData, uint32 InSrcBufferDataSize, struct FSoundQualityInfo* QualityInfo)
{
	if (CompressedAudioInfo)
	{
		return CompressedAudioInfo->ReadCompressedInfo(InSrcBufferData, InSrcBufferDataSize, QualityInfo);
	}
	return false;
}

bool FSoundWaveProxyDecodeCacheDecoder::ReadCompressedData(uint8* Destination, bool bLooping, uint32 BufferSize)
{
	int32 NumBytesRead = 0;
	return GetCompressedDataInternal(Destination, bLooping, BufferSize, NumBytesRead, false);
}

void FSoundWaveProxyDecodeCacheDecoder::SeekToTime(const float SeekTime)
{
	if (NumFramesInWave > 0)
	{
		float SeekFrame = FMath::Clamp(static_cast<int32>(SeekTime * SampleRate), 0, NumFramesInWave);
		SeekToFrame(SeekFrame);
	}
	else
	{
		CurrentReadFrameIndex = 0;
		DecodedSoundWaveChunk = nullptr;
	}
}

void FSoundWaveProxyDecodeCacheDecoder::SeekToFrame(const uint32 Frame)
{
	LLM_SCOPE_BYTAG(Audio_SoundWaveDecodeCache);

	CurrentReadFrameIndex = Frame;

	// Check to see if our seek is outside the range of a current chunk
	if (DecodedSoundWaveChunk.IsValid())
	{
		if (CurrentReadFrameIndex < DecodedSoundWaveChunk->FrameStart || CurrentReadFrameIndex >= DecodedSoundWaveChunk->FrameEnd)
		{
			DecodedSoundWaveChunk = nullptr;
		}
	}
}

void FSoundWaveProxyDecodeCacheDecoder::ExpandFile(uint8* DstBuffer, struct FSoundQualityInfo* QualityInfo)
{
	if (CompressedAudioInfo)
	{
		CompressedAudioInfo->ExpandFile(DstBuffer, QualityInfo);
	}
}

void FSoundWaveProxyDecodeCacheDecoder::EnableHalfRate(bool HalfRate)
{
	if (CompressedAudioInfo)
	{
		CompressedAudioInfo->EnableHalfRate(HalfRate);
	}
}

uint32 FSoundWaveProxyDecodeCacheDecoder::GetSourceBufferSize() const
{
	if (CompressedAudioInfo)
	{
		return CompressedAudioInfo->GetSourceBufferSize();
	}
	return 0;
}

bool FSoundWaveProxyDecodeCacheDecoder::UsesVorbisChannelOrdering() const
{
	if (CompressedAudioInfo)
	{
		return CompressedAudioInfo->UsesVorbisChannelOrdering();
	}
	return false;
}

int32 FSoundWaveProxyDecodeCacheDecoder::GetStreamBufferSize() const
{
	if (CompressedAudioInfo)
	{
		return CompressedAudioInfo->GetSourceBufferSize();
	}
	return 0;
}

bool FSoundWaveProxyDecodeCacheDecoder::HasError() const
{
	if (CompressedAudioInfo)
	{
		return CompressedAudioInfo->HasError();
	}
	return true;
}

bool FSoundWaveProxyDecodeCacheDecoder::StreamCompressedData(uint8* Destination, bool bLooping, uint32 BufferSize, int32& OutNumBytesStreamed)
{
	return GetCompressedDataInternal(Destination, bLooping, BufferSize, OutNumBytesStreamed, true);
}

int32 FSoundWaveProxyDecodeCacheDecoder::GetCurrentChunkIndex() const
{
	if (CompressedAudioInfo)
	{
		return CompressedAudioInfo->GetCurrentChunkIndex();
	}
	return INDEX_NONE;
}

int32 FSoundWaveProxyDecodeCacheDecoder::GetCurrentChunkOffset() const
{ 
	if (CompressedAudioInfo)
	{
		return CompressedAudioInfo->GetCurrentChunkOffset();
	}
	return INDEX_NONE;
}

const TSharedPtr<const FSoundWaveData>& FSoundWaveProxyDecodeCacheDecoder::GetStreamingSoundWaveData() const
{
	if (CompressedAudioInfo)
	{
		return CompressedAudioInfo->GetStreamingSoundWaveData();
	}
	static const TSharedPtr<const FSoundWaveData> Empty;
	return Empty;
}

bool FSoundWaveProxyDecodeCacheDecoder::StreamCompressedInfoInternal(const TSharedRef<const FSoundWaveData>& InWaveData, struct FSoundQualityInfo* QualityInfo)
{
	if (CompressedAudioInfo)
	{
		return CompressedAudioInfo->StreamCompressedInfo(InWaveData, QualityInfo);
	}
	return false;
}

bool FSoundWaveProxyDecodeCacheDecoder::GetCompressedDataInternal(uint8* Destination, bool bLooping, uint32 BufferSize, int32& OutNumBytesStreamed, bool bDoStreamingDecode)
{
	if (!CompressedAudioInfo)
	{
		return false;
	}

	check(NumChannels > 0);
	check(SampleRate > 0);

	bool bIsFinished = false;

	// The number of bytes we need to still copy out. 
	int32 NumFramesToCopyOut = (int32)BufferSize / (MONO_PCM_SAMPLE_SIZE * NumChannels);
	OutNumBytesStreamed = 0;

	while (NumFramesToCopyOut > 0 && !bIsFinished)
	{
		// Check to see if we already have a decoded chunk 
		if (DecodedSoundWaveChunk.IsValid())
		{
			int32 ChunkFrameStart = DecodedSoundWaveChunk->FrameStart;
			int32 ChunkFrameEnd = DecodedSoundWaveChunk->FrameEnd;
			int32 ChunkFrameReadIndex = CurrentReadFrameIndex - ChunkFrameStart;
			int32 NumFramesLeftInThisChunk = FMath::Max<int32>(ChunkFrameEnd - CurrentReadFrameIndex, 0);
			int32 NumFramesToCopyFromThisChunk = FMath::Min(NumFramesToCopyOut, NumFramesLeftInThisChunk);

			// if we have frames to copy out from this decoded buffer, we don't need to get another chunk
			if (NumFramesToCopyFromThisChunk > 0)
			{
				check(NumFramesToCopyFromThisChunk + ChunkFrameStart <= ChunkFrameEnd);
				int32 NumBytesToCopy = MONO_PCM_SAMPLE_SIZE * NumFramesToCopyFromThisChunk * NumChannels;
				int32 CopyFromByteOffset = MONO_PCM_SAMPLE_SIZE * ChunkFrameReadIndex * NumChannels;
				uint8* CopyFromBufferPtr = DecodedSoundWaveChunk->DecodedByteData.GetData();
				int32 NumBytesInDecodeBuffer = DecodedSoundWaveChunk->DecodedByteData.Num();

				uint8* Dest = Destination + OutNumBytesStreamed;
				int32 RemainingBytesInDest = BufferSize - OutNumBytesStreamed;
				check(NumBytesToCopy <= RemainingBytesInDest);

				uint8* Src = CopyFromBufferPtr + CopyFromByteOffset;
				check(CopyFromByteOffset + NumBytesToCopy <= NumBytesInDecodeBuffer);
				FMemory::Memcpy(Dest, Src, NumBytesToCopy);

				OutNumBytesStreamed += NumBytesToCopy;
				NumFramesToCopyOut -= NumFramesToCopyFromThisChunk;
				CurrentReadFrameIndex += NumFramesToCopyFromThisChunk;
			}

			// Check for finished condition
			// If we are not looping and if we've reached the end of this chunk, and it's the last chunk
			if (NumFramesToCopyFromThisChunk == NumFramesLeftInThisChunk && DecodedSoundWaveChunk->bIsLastChunk && !bLooping)
			{
				bIsFinished = true;
				break;
			}
		}

		// we still have frames we need to copy out, so get a new decoded buffer from the cache
		// OR if there is no decoded audio in the cache for the data we need, do the decode for a new chunk
		if (NumFramesToCopyOut > 0 && !bIsFinished)
		{
			using namespace SoundWaveProxyDecodeCache;

			// Retrieve the global decode cache
			FSoundWaveProxyDecodeCache* DecodeCache = FSoundWaveProxyDecodeCache::Get();
			check(DecodeCache);
			
			int32 FramesPerChunk = DecodeCache->GetFramesPerChunk();
			check(FramesPerChunk > 0);

			// build a decode buffer key that we want to query
			FDecodedSoundWaveChunkKey NewChunkKey;
			NewChunkKey.WaveName = WaveName;
			NewChunkKey.WaveGuid = WaveGuid;
			NewChunkKey.ChunkIndex = CurrentReadFrameIndex / FramesPerChunk;
#if WITH_EDITOR
			NewChunkKey.ChunkRevision = ChunkRevision;
			NewChunkKey.NumChannels = NumChannels;
			NewChunkKey.SampleRate = SampleRate;
#endif
			DecodedSoundWaveChunk = DecodeCache->GetDecodedChunk(NewChunkKey);

			// if there is no decoded chunk for this key, that means we need to do the decode
			// with our underlying contained CompressedAudioInfo object
			if (!DecodedSoundWaveChunk.IsValid())
			{
				int32 SeekFrame = NewChunkKey.ChunkIndex * FramesPerChunk;
				int32 BytesPerChunk = FramesPerChunk * MONO_PCM_SAMPLE_SIZE * NumChannels;

				{
					LLM_SCOPE_BYTAG(Audio_SoundWaveDecodeCache);

					DecodedSoundWaveChunk = MakeShared<FDecodedSoundWaveChunk>();
					DecodedSoundWaveChunk->FrameStart = SeekFrame;
					DecodedSoundWaveChunk->DecodedByteData.AddUninitialized(BytesPerChunk);
				}

				// first, seek the underlying decoder to the required chunk boundary
				check(CompressedAudioInfo);
				CompressedAudioInfo->SeekToFrame(SeekFrame);

				uint8* DecodedByteDataPtr = DecodedSoundWaveChunk->DecodedByteData.GetData();

				int32 NumBytesStreamed = 0;
				bool bIsLastDecodeChunk = false;
				if (bDoStreamingDecode)
				{
					bIsLastDecodeChunk = CompressedAudioInfo->StreamCompressedData(DecodedByteDataPtr, bLooping, BytesPerChunk, NumBytesStreamed);
				}
				else
				{
					NumBytesStreamed = BytesPerChunk;
					bIsLastDecodeChunk = CompressedAudioInfo->ReadCompressedData(DecodedByteDataPtr, bLooping, BytesPerChunk);
				}

				// No data was decoded, so early out
				if (NumBytesStreamed == 0)
				{
					break;
				}

				check(bIsLastDecodeChunk || NumBytesStreamed == BytesPerChunk);
				DecodedSoundWaveChunk->bIsLastChunk = bIsLastDecodeChunk;
				DecodedSoundWaveChunk->FrameEnd = DecodedSoundWaveChunk->FrameStart + (NumBytesStreamed / (MONO_PCM_SAMPLE_SIZE * NumChannels));

				int32 FramesInChunk = DecodedSoundWaveChunk->FrameEnd - DecodedSoundWaveChunk->FrameStart;
				check((FramesInChunk * (int32)MONO_PCM_SAMPLE_SIZE * (int32)NumChannels) <= (int32)BytesPerChunk);
				check(FramesInChunk <= FramesPerChunk);

				DecodeCache->CacheDecodedChunk(NewChunkKey, DecodedSoundWaveChunk);
			}
		}
	}

	return bIsFinished;
}

