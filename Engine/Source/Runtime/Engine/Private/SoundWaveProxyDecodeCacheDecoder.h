// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Sound/SoundWaveProxyDecodeCache.h"
#include "SoundWaveProxyDecodeCacheImpl.h"
#include "Interfaces/IAudioFormat.h"
#include "AudioDecompress.h"

class FSoundWaveProxyDecodeCache;
struct FDecodedSoundWaveChunk;

/*
 * Concrete implementation of ICompressedAudioInfo, which wraps an underlying
 * decoder and intercepts decode and seek calls to translate it to interactions with a decode cache.
 */
class FSoundWaveProxyDecodeCacheDecoder : public ICompressedAudioInfo
{
public:
	using FSoundWaveProxyRef = TSharedRef<FSoundWaveProxy, ESPMode::ThreadSafe>;
	using FSoundWaveDataRef = TSharedRef<const FSoundWaveData, ESPMode::ThreadSafe>;
	using FDecodedSoundWaveChunkPtr = TSharedPtr<FDecodedSoundWaveChunk, ESPMode::ThreadSafe>;

	FSoundWaveProxyDecodeCacheDecoder(const FSoundWaveDataRef& InWaveData);

	UE_DEPRECATED(5.8, "Use the TSharedRef<const FSoundWaveData>& overload instead.")
	FSoundWaveProxyDecodeCacheDecoder(const FSoundWaveProxyRef& InWaveProxy);

	virtual ~FSoundWaveProxyDecodeCacheDecoder()
	{}

	virtual bool ReadCompressedInfo(const uint8* InSrcBufferData, uint32 InSrcBufferDataSize, struct FSoundQualityInfo* QualityInfo) override final;
	virtual bool ReadCompressedData(uint8* Destination, bool bLooping, uint32 BufferSize) override final;
	virtual void SeekToTime(const float SeekTime) override final;
	virtual void SeekToFrame(const uint32 Frame) override final;
	virtual void ExpandFile(uint8* DstBuffer, struct FSoundQualityInfo* QualityInfo) override final;
	virtual void EnableHalfRate(bool HalfRate) override final;
	virtual uint32 GetSourceBufferSize() const override final;
	virtual bool UsesVorbisChannelOrdering() const override final;
	virtual int32 GetStreamBufferSize() const override final;
	virtual bool HasError() const override final;
	virtual bool StreamCompressedData(uint8* Destination, bool bLooping, uint32 BufferSize, int32& OutNumBytesStreamed) override final;
	virtual int32 GetCurrentChunkIndex() const override final;
	virtual int32 GetCurrentChunkOffset() const override final;
	virtual const TSharedPtr<const FSoundWaveData>& GetStreamingSoundWaveData() const override final;

protected:
	virtual bool StreamCompressedInfoInternal(const TSharedRef<const FSoundWaveData>& InWaveData, struct FSoundQualityInfo* QualityInfo) override final;

private:

	bool GetCompressedDataInternal(uint8* Destination, bool bLooping, uint32 BufferSize, int32& OutNumBytesStreamed, bool bDoStreamingDecode);

	// The actual decoder we are using for decoding
	TUniquePtr<ICompressedAudioInfo> CompressedAudioInfo;

	// A ptr to the decoded chunk of sound wave data
	// This is either decoded on the fly and discarded OR pulled from the decode cache
	// if this decoder is flagged to use the decode cache
	FDecodedSoundWaveChunkPtr DecodedSoundWaveChunk;

	uint32 NumChannels = 0;
	uint32 SampleRate = 0;
	int32 NumFramesInWave = INDEX_NONE;
	int32 CurrentReadFrameIndex = 0;
#if WITH_EDITOR
	int32 ChunkRevision = INDEX_NONE;
#endif
	FName WaveName;
	FGuid WaveGuid;
};
