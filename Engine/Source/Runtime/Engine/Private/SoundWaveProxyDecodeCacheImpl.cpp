// Copyright Epic Games, Inc. All Rights Reserved.

#include "SoundWaveProxyDecodeCacheImpl.h"
#include "Math/UnitConversion.h"
#include "SoundWaveProxyDecodeCacheDecoder.h"
#include "HAL/IConsoleManager.h"


static int32 EnableDecodeCacheCvar = -1;
FAutoConsoleVariableRef CVarEnableDecodeCache(
	TEXT("au.DecodeCache.Enabled"),
	EnableDecodeCacheCvar,
	TEXT("Overrides project settings to enable or disable decode cache.\n")
	TEXT("-1: Use project settings value, 0: Disable, 1: Enable"),
	ECVF_Default);

static float AutoDurationThresholdCvar = -1.0f;
FAutoConsoleVariableRef CVarAutoDurationThreshold(
	TEXT("au.DecodeCache.DurationThreshold"),
	AutoDurationThresholdCvar,
	TEXT("If non-negative, overrides project settings to set the auto duration threshold. Sounds with duration below this number will use the decode cache automatically.\n")
	TEXT("-1: Use project settings value, >= 0.0 will use this value for auto duration threshold. A value of 0.0 means every sound will use decode cache if it is enabled."),
	ECVF_Default);


LLM_DEFINE_TAG(Audio_SoundWaveDecodeCache);

FSoundWaveProxyDecodeCache::FSoundWaveProxyDecodeCache()
{
	const UAudioSettings* Settings = GetDefault<UAudioSettings>();
	check(Settings);

	const FDecodeCacheSettings& DecodeCacheSettings = Settings->GetDecodeCacheSettings();

	// Clamp settings to reasonable limits to prevent out-of-bounds issues
	DesiredCacheSizeMB = FMath::Clamp(DecodeCacheSettings.MaxDecodeCacheSizeMB, 1.0f, 100.0f);
	DesiredFramesPerChunk = FMath::Clamp(DecodeCacheSettings.FramesPerChunk, 1024, 48000);
	AutoDurationThreshold = FMath::Max(DecodeCacheSettings.AutoDurationThreshold, 0.0f);

	float DesiredCacheSizeBytes = FUnitConversion::Convert(DesiredCacheSizeMB, EUnit::Megabytes, EUnit::Bytes);

	// Conform the requested frames per chunk from the settings to one that will actually work
	FramesPerChunk = ICompressedAudioInfo::ConformDecodeSize(DesiredFramesPerChunk);

	// Calculate the worse-case lru chunk count based on mono buffers of the desired min frames per chunk and the size of the struct used to 
	// track the data in the chunk entries
	int32 TotalBytesPerEntry = MONO_PCM_SAMPLE_SIZE * FramesPerChunk + sizeof(FDecodedSoundWaveChunk) + sizeof(FDecodedSoundWaveChunkKey);
	MaxChunkCount = DesiredCacheSizeBytes / TotalBytesPerEntry;

	// Lets set some extreme but minimal chunk count limit
	constexpr int32 MinChunkCount = 10;
	if (MaxChunkCount > MinChunkCount)
	{
		check(bHasFailure == false);
		MaxMemoryBytes = MaxChunkCount * TotalBytesPerEntry;

		// Since there could be rounding issues with fitting the desired budget to frames per chunk, our actual budget may differ slightly
		float ActualMaxMemoryMB = FUnitConversion::Convert((float)MaxMemoryBytes, EUnit::Bytes, EUnit::Megabytes);

		UE_LOGF(LogAudio, Display,
			"Initializing decode cache to %.2f Megabytes (Desired: %.2f), MaxChunkCount: %d. AutoDurationThreshold: %.2f, Enabled: %ls",
			ActualMaxMemoryMB, DesiredCacheSizeMB, MaxChunkCount, AutoDurationThreshold, SoundWaveProxyDecodeCache::IsEnabled() ? TEXT("Yes") : TEXT("No"));

		// Note that we are tracking the LRU overhead here since the LRU cache is pre-allocated when initializing.
		UsedMemoryBytes = MaxChunkCount * (sizeof(FDecodedSoundWaveChunk) + sizeof(FDecodedSoundWaveChunkKey));
		check(NumChunksUsed == 0);

		LLM_SCOPE_BYTAG(Audio_SoundWaveDecodeCache);

		// Initialize the Lru to the max chunk count (worse-case all mono audio)
		LruCache.Empty(MaxChunkCount);
	}
	else
	{
		bHasFailure = true;

		UE_LOGF(LogAudio, Warning, 
			"Failed to initialize the decode cache due to settings (MaxDecodeCacheSizeMB: %.2f, FramesPerChunk: %d) which result in extremely small decode chunk count: %d (Min Chunk Count: %d).", 
			DecodeCacheSettings.MaxDecodeCacheSizeMB, DecodeCacheSettings.FramesPerChunk, MaxChunkCount, MinChunkCount);
	}
}

int32 FSoundWaveProxyDecodeCache::GetFramesPerChunk() const
{
	check(FramesPerChunk > 0);
	return FramesPerChunk;
}

FSoundWaveProxyCacheInfo FSoundWaveProxyDecodeCache::GetInfo() const
{
	FSoundWaveProxyCacheInfo Info;
	Info.MaxMemoryBytes = MaxMemoryBytes;
	Info.NumChunksCapacity = MaxChunkCount;
	Info.UsedChunks = NumChunksUsed;
	Info.UsedMemoryBytes = UsedMemoryBytes;
	return Info;
}

float FSoundWaveProxyDecodeCache::GetAutoDurationThreshold() const
{
	if (AutoDurationThresholdCvar >= 0.0f)
	{
		return AutoDurationThresholdCvar;
	}

	return AutoDurationThreshold;
}


TUniquePtr<ICompressedAudioInfo> FSoundWaveProxyDecodeCache::CreateDecoderInstance(const FSoundWaveDataRef& InSoundWaveData)
{
	LLM_SCOPE_BYTAG(Audio_SoundWaveDecodeCache);

	return MakeUnique<FSoundWaveProxyDecodeCacheDecoder>(InSoundWaveData);
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
TUniquePtr<ICompressedAudioInfo> FSoundWaveProxyDecodeCache::CreateDecoderInstance(const FSoundWaveProxyRef& InSoundWaveProxy)
{
	// Forward to the data-ref overload; the proxy is just a broker for FSoundWaveData.
	return CreateDecoderInstance(InSoundWaveProxy->GetSoundWaveDataRef());
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

FSoundWaveProxyDecodeCache::FDecodedSoundWaveChunkPtr FSoundWaveProxyDecodeCache::GetDecodedChunk(const FDecodedSoundWaveChunkKey& InKey)
{
	FScopeLock Lock(&CacheLock);

	if (SoundWaveProxyDecodeCache::IsEnabled() && !bHasFailure)
	{
		FDecodedSoundWaveChunkPtr* Entry = LruCache.FindAndTouch(InKey);
		if (Entry)
		{
			return *Entry;
		}
	}

	return nullptr;
}

void FSoundWaveProxyDecodeCache::CacheDecodedChunk(const FDecodedSoundWaveChunkKey& InKey, FDecodedSoundWaveChunkPtr& InDecodedChunk)
{
	FScopeLock Lock(&CacheLock);

	LLM_SCOPE_BYTAG(Audio_SoundWaveDecodeCache);

	// Don't allow caching chunks if we're disabled or have a failure
	if (!SoundWaveProxyDecodeCache::IsEnabled() || bHasFailure)
	{
		return;
	}

	// First check if chunk is already in the cache. If it is, we'll touch it and early return
	FDecodedSoundWaveChunkPtr* AlreadyInCache = LruCache.FindAndTouch(InKey);
	if (AlreadyInCache)
	{
		return;
	}

	int32 NextUsedMemoryBytes = UsedMemoryBytes + InDecodedChunk->DecodedByteData.Num() + sizeof(FDecodedSoundWaveChunk) + sizeof(FDecodedSoundWaveChunkKey);

	// Only add the decoded chunk to the cache if it's below our budget
	// If try to trim the cache down to below our budget
	bool bCanCacheChunk = SoundWaveProxyDecodeCache::IsEnabled();
	while (bCanCacheChunk && NextUsedMemoryBytes >= MaxMemoryBytes)
	{
		// The first chunk may exceed the budget and try to evict from an empty cache which will cause an assert
		if (!LruCache.IsEmpty())
		{
			FDecodedSoundWaveChunkKey LeastRecentKey = LruCache.GetLeastRecentKey();
			FDecodedSoundWaveChunkPtr& LeastRecentChunk = LruCache.FindChecked(LeastRecentKey);

			// if the least-recent entry only has one reference (it's unique), then this decode cache is it's only reference and it's safe to remove
			if (LeastRecentChunk.IsUnique())
			{
				int32 TotalMemSizeOfLeastRecentEntry = LeastRecentChunk->DecodedByteData.Num() + sizeof(FDecodedSoundWaveChunk) + sizeof(FDecodedSoundWaveChunkKey);
				NextUsedMemoryBytes -= TotalMemSizeOfLeastRecentEntry;
				NumChunksUsed--;
				LruCache.RemoveLeastRecent();
			}
			else
			{
				// Our least recently used chunk is being use, which means the cache has no chunks that can be evicted so we can't use the decode cache
				bCanCacheChunk = false;
			}
			// Check if we need to terminate this loop early because the cache is totally full and everything is being used (i.e. we can't cache this chunk)
		}
		else
		{
			bCanCacheChunk = false;
		}
	}

	if (bCanCacheChunk)
	{
		UsedMemoryBytes = NextUsedMemoryBytes;
		NumChunksUsed++;
		LruCache.Add(InKey, InDecodedChunk);
	}
	else
	{
		UE_LOGF(LogAudio, Verbose, 
			"FSoundWaveProxyDecodeCache::CacheDecodedChunk, Failed to cache decoded audio chunk: cache is full and all chunks are referenced. NumChunksUsed: %d", NumChunksUsed);
	}
}

FSoundWaveProxyDecodeCache* FSoundWaveProxyDecodeCache::Get()
{
	static TUniquePtr<FSoundWaveProxyDecodeCache> DecodeCache = MakeUnique<FSoundWaveProxyDecodeCache>();
	return DecodeCache.Get();
}

bool SoundWaveProxyDecodeCache::IsEnabled()
{
	// -1 means to use the project settings
	if (EnableDecodeCacheCvar == -1)
	{
		// Only access settings once and cache the results
		auto IsCacheEnabledFromSettings = []() -> bool
			{
				const UAudioSettings* Settings = GetDefault<UAudioSettings>();
				if (Settings)
				{
					const FDecodeCacheSettings& DecodeCacheSettings = Settings->GetDecodeCacheSettings();
					return DecodeCacheSettings.EnableDecodeCache;
				}
				return false;
			};

		static const bool IsEnabledCached = IsCacheEnabledFromSettings();
		return IsEnabledCached;
	}
	else
	{
		// otherwise 0 means not enabled, >= 1 means enabled
		return (EnableDecodeCacheCvar != 0);
	}
}

TUniquePtr<ICompressedAudioInfo> SoundWaveProxyDecodeCache::CreateDecoderInstance(const FSoundWaveDataRef& InSoundWaveData)
{
	FSoundWaveProxyDecodeCache* DecodeCache = FSoundWaveProxyDecodeCache::Get();
	float AutoDurationThresholdHold = DecodeCache->GetAutoDurationThreshold();
	if (InSoundWaveData->GetDuration() <= AutoDurationThresholdHold)
	{
		return DecodeCache->CreateDecoderInstance(InSoundWaveData);
	}
	return nullptr;
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
TUniquePtr<ICompressedAudioInfo> SoundWaveProxyDecodeCache::CreateDecoderInstance(const FSoundWaveProxyRef& InSoundWaveProxy)
{
	return CreateDecoderInstance(InSoundWaveProxy->GetSoundWaveDataRef());
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

