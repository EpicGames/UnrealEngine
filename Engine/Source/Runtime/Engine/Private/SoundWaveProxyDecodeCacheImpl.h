// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Audio.h"
#include "Containers/LruCache.h"
#include "Math/UnitConversion.h"
#include "Sound/SoundWaveProxyDecodeCache.h"
#include "SoundWaveProxyDecodeCacheDecoder.h"
#include "Templates/SharedPointer.h"

class FSoundWaveProxyDecodeCacheDecoder;

// Key used to uniquely look up already decoded chunks of PCM data from wave proxies
struct FDecodedSoundWaveChunkKey
{
	// The name of the wave proxy (useful for debugging)
	FName WaveName;

	// The guid of the wave is used for life-time independent cache key
	FGuid WaveGuid;

	// The chunk index of the decoded chunk, based on the current frame index and chunk size
	uint32 ChunkIndex = INDEX_NONE;

	// Prevents invalid cache hits in editor in the case of re-imported or other format changes
#if WITH_EDITOR
	uint32 ChunkRevision = 0;
	uint32 NumChannels = 0;
	uint32 SampleRate = 0;
#endif

	bool operator==(const FDecodedSoundWaveChunkKey& Other) const
	{
		return WaveName == Other.WaveName
			&& WaveGuid == Other.WaveGuid
			&& ChunkIndex == Other.ChunkIndex
#if WITH_EDITOR
			&& ChunkRevision == Other.ChunkRevision
			&& NumChannels == Other.NumChannels
			&& SampleRate == Other.SampleRate
#endif
			;
	}

	friend uint32 GetTypeHash(const FDecodedSoundWaveChunkKey& Key)
	{
		uint32 Hash = HashCombineFast(GetTypeHash(Key.WaveName), Key.ChunkIndex);
		Hash = HashCombineFast(Hash, GetTypeHash(Key.WaveGuid));
#if WITH_EDITOR
		Hash = HashCombineFast(Hash, Key.ChunkRevision);
		Hash = HashCombineFast(Hash, Key.NumChannels);
		Hash = HashCombineFast(Hash, Key.SampleRate);
#endif
		return Hash;
	}
};

/**
 * The decoded audio data struct which is cached in the sound wave proxy decode cache
 */
struct FDecodedSoundWaveChunk
{
	// The interleaved audio frame that this chunk starts
	int32 FrameStart = 0;

	// The interleaved frame this chunk ends. Note: this could be less than data size due to it being the last chunk.
	int32 FrameEnd = 0;

	// If the file is finished in this chunk
	bool bIsLastChunk = false;

	// The interleaved decoded audio PCM byte data
	TArray<uint8> DecodedByteData;
};

/**
 * Struct holding information about the decode cache.
 */
struct FSoundWaveProxyCacheInfo
{
	// The max number of chunks possible in the cache (if everything was mono)
	int32 NumChunksCapacity = 0;
	// The number of chunks currently being used
	int32 UsedChunks = 0;
	// The number of bytes currently being used
	int32 UsedMemoryBytes = 0;
	// The max memory allowed by the cache
	int32 MaxMemoryBytes = 0.0f;
};


/**
 * A thread-safe decode cache for use with wave proxies. Allows avoiding duplicate decoding of audio which is replayed often.
 */
class FSoundWaveProxyDecodeCache
{
public:
	using FSoundWaveProxyRef = TSharedRef<FSoundWaveProxy, ESPMode::ThreadSafe>;
	using FSoundWaveDataRef = TSharedRef<const FSoundWaveData, ESPMode::ThreadSafe>;
	using FDecodedSoundWaveChunkPtr = TSharedPtr<FDecodedSoundWaveChunk, ESPMode::ThreadSafe>;

	FSoundWaveProxyDecodeCache();
	~FSoundWaveProxyDecodeCache() {};

	int32 GetFramesPerChunk() const;
	FSoundWaveProxyCacheInfo GetInfo() const;
	float GetAutoDurationThreshold() const;
	TUniquePtr<ICompressedAudioInfo> CreateDecoderInstance(const FSoundWaveDataRef& InSoundWaveData);

	UE_DEPRECATED(5.8, "Use the TSharedRef<const FSoundWaveData>& overload instead.")
	TUniquePtr<ICompressedAudioInfo> CreateDecoderInstance(const FSoundWaveProxyRef& InSoundWaveProxy);

	/**
	* Returns the private singleton implementation of the sound wave proxy decode cache.
	* If this is the first time the decode cache is retrieved, it will construct and initialize it.
	*/
	static FSoundWaveProxyDecodeCache* Get();

private:

	// Retrieve already decoded chunk with the given key
	FDecodedSoundWaveChunkPtr GetDecodedChunk(const FDecodedSoundWaveChunkKey& InKey);

	// Cache the given decoded audio into the decode cache
	void CacheDecodedChunk(const FDecodedSoundWaveChunkKey& InKey, FDecodedSoundWaveChunkPtr& InDecodedChunk);

	// A cache container that stores in-use chunks in a least-recently-used list automatically.
	TLruCache<FDecodedSoundWaveChunkKey, FDecodedSoundWaveChunkPtr> LruCache;

	// Settings used when initializing the decode cache
	bool bHasFailure = false;
	float DesiredCacheSizeMB = 0.0f;
	float AutoDurationThreshold = 0.0f;
	int32 DesiredFramesPerChunk = 0;

	// The worse-case max chunk count for the decode cache. 
	int32 MaxChunkCount = 0;

	// The number of chunks currently being used from the TLruCache
	int32 NumChunksUsed = 0;

	// The actual max budget in bytes
	int32 MaxMemoryBytes = 0;

	// The actual frames per chunk
	int32 FramesPerChunk = 0;

	// The bytes currently being used, used to trigger a trim automatically
	int32 UsedMemoryBytes = 0;

	// Critical section for accessing the cache
	mutable FCriticalSection CacheLock;

	friend class FSoundWaveProxyDecodeCacheDecoder;
};

