// Copyright Epic Games, Inc. All Rights Reserved.

#include "Sound/SoundWaveProxyPlayer.h"

#include "Audio.h"
#include "AudioDecompress.h"
#include "DSP/FloatArrayMath.h"
#include "Interfaces/IAudioFormat.h"
#include "ProfilingDebugging/CpuProfilerTrace.h"
#include "Sound/SoundWaveProxyDecodeCache.h"
#include "Subtitles/SubtitlesAndClosedCaptionsDelegates.h"
#include "Templates/SharedPointer.h"

FSoundWaveProxyPlayer::FSoundWaveProxyPlayer(const FSettings& InSettings)
	: Settings(InSettings)
	, RuntimeResampler(1)
{

}

bool FSoundWaveProxyPlayer::SetSoundWave(const TSharedRef<const FSoundWaveData>& InWaveData)
{ 
	SoundWaveDataPtr = InWaveData.ToSharedPtr();
	check(SoundWaveDataPtr);
	
	if (!IsValidSoundWaveData(*SoundWaveDataPtr))
	{
		UE_LOG(LogAudio, Warning, TEXT("Failed to Set Sound Wave in FSoundWaveProxyPlayer(), Package: %s"), *WriteToString<64>(SoundWaveDataPtr->GetPackageName()));
		Reset(Settings);
		return false;
	}

	if (!InitializeDecoder(Settings.MaxDecodeSizeFrames))
	{
		UE_LOG(LogAudio, Warning, TEXT("Failed to InitializeDecoder in FSoundWaveProxyPlayer(), Package: %s"), *WriteToString<64>(SoundWaveDataPtr->GetPackageName()));
		Reset(Settings);
		return false;
	}
	SourceAudio.Reset();
	SourceFrameIndex = 0;
	AudioSyncFrameIndex = 0;
	// update the loop with the cached loop settings to update the loop start/end frames
	SetLoop(bIsLooping, CurrentLoopStartTime, CurrentLoopDuration);
	bFallbackSeekMethodWarningLogged = false;
	bHasNearlyFinished = false;
	SortedCuePoints = SoundWaveDataPtr->GetCuePointsAndLoopRegions();
	Algo::SortBy(SortedCuePoints, &FSoundWaveCuePoint::FramePosition);

	InitializeResampler(NumChannels);
	
	// Create algorithm for channel conversion and deinterleave 
	Audio::FConvertDeinterleaveParams ConvertDeinterleaveParams;
	ConvertDeinterleaveParams.NumInputChannels = NumChannels;
	ConvertDeinterleaveParams.NumOutputChannels = Settings.OutputNumChannels;
	ConvertDeinterleaveParams.MonoUpmixMethod = Settings.MonoUpmixMethod;
	ConvertDeinterleave = Audio::IConvertDeinterleave::Create(ConvertDeinterleaveParams);

	Chan0SamplesSinceQueuedSubtitles = 0;

	return true;
}


bool FSoundWaveProxyPlayer::SetSoundWave(FSoundWaveProxyPtr InWaveProxy)
{
	if (!InWaveProxy.IsValid())
	{
		Reset(Settings);
		return false;
	}
	
	return SetSoundWave(InWaveProxy->GetSoundWaveDataRef());
}

int32 FSoundWaveProxyPlayer::GetOutputNumChannels() const
{
	return Settings.OutputNumChannels;
}

float FSoundWaveProxyPlayer::GetOutputSampleRate() const
{
	return Settings.OutputSampleRate;
}

int32 FSoundWaveProxyPlayer::GetSourceNumChannels() const
{
	check(IsPlayerValid());
	return NumChannels;
}

float FSoundWaveProxyPlayer::GetSourceSampleRate() const
{
	check(IsPlayerValid())
	return SampleRate;
}

int32 FSoundWaveProxyPlayer::GetSourceNumFrames() const
{
	check(IsPlayerValid());
	return NumFramesInWave;	
}

float FSoundWaveProxyPlayer::GetSourceDuration() const
{
	check(IsPlayerValid())
	return DurationSeconds;
}

bool FSoundWaveProxyPlayer::IsPlayerValid() const
{
	return SoundWaveDataPtr.IsValid() && CompressedAudioInfo.IsValid();
}

void FSoundWaveProxyPlayer::Reset(const FSettings& NewSettings)
{
	SoundWaveDataPtr.Reset();
	Settings = NewSettings;
	DecodeBuffer.Reset();
	CompressedAudioInfo.Reset();
	ConvertDeinterleave.Reset();
	SourceAudio.Reset();
	ResampledAudio.Reset();
	SourceFrameIndex = 0;
	AudioSyncFrameIndex = 0;
	Speed = 1.0f;
	bIsLooping = false;
	LoopStartFrameIndex = 0;
	LoopEndFrameIndex = -1;
	bRequiresResampling = false;
	bFallbackSeekMethodWarningLogged = false;
	bHasNearlyFinished = false;

	SampleRate = -1.0f;
	NumChannels = -1;
	NumFramesInWave = -1;
	DurationSeconds = -1.0f;
	SortedCuePoints.Reset();

	Chan0SamplesSinceQueuedSubtitles = 0;
}

TUniquePtr<FSoundWaveProxyPlayer> FSoundWaveProxyPlayer::Create(const FSettings& InSettings)
{
	return TUniquePtr<FSoundWaveProxyPlayer>(new FSoundWaveProxyPlayer(InSettings));
}

bool FSoundWaveProxyPlayer::IsValidSoundWaveData(const FSoundWaveData& InWaveData)
{
	if (InWaveData.GetSampleRate() <= 0.f)
	{
		UE_LOG(LogAudio, Warning, TEXT("Cannot create FSoundWaveProxyPlayer due to invalid sample rate (%f). Package: %s"), InWaveData.GetSampleRate(), *InWaveData.GetPackageName().ToString());
		return false;
	}

	if (InWaveData.GetNumChannels() <= 0)
	{
		UE_LOG(LogAudio, Warning, TEXT("Cannot create FSoundWaveProxyPlayer due to invalid num channels (%d). Package: %s"), InWaveData.GetNumChannels(), *InWaveData.GetPackageName().ToString());
		return false;
	}

	if (InWaveData.GetNumFrames() <= 0)
	{
		UE_LOG(LogAudio, Warning, TEXT("Cannot create FSoundWaveProxyPlayer due to invalid num frames (%d). Package: %s"), InWaveData.GetNumFrames(), *InWaveData.GetPackageName().ToString());
		return  false;
	}

	return true;
}

void FSoundWaveProxyPlayer::GenerateSourceEvents(int32 BlockFrameIndex, int32 NumFrames, TArray<FSourceEvent>& OutEvents)
{
	check(SoundWaveDataPtr.IsValid());

	if (IsFinished())
	{
		return;
	}

	float FrameRatio = GetCurrentFrameRatio();
	if (bRequiresResampling || !FMath::IsNearlyEqual(FrameRatio, 1.0f))
	{
		bRequiresResampling = true;
		RuntimeResampler.SetFrameRatio(FrameRatio);
		NumFrames = RuntimeResampler.GetNumInputFramesNeededToProduceOutputFrames(NumFrames);
	}
	
	int32 StartFrameIndex = Settings.bMaintainAudioSync ? AudioSyncFrameIndex : SourceFrameIndex;

	if (!bIsLooping)
	{
		if (StartFrameIndex >= NumFramesInWave)
		{
			return;
		}
		
		GenerateSourceEventsInternal(StartFrameIndex, BlockFrameIndex, NumFrames, OutEvents);

		// check if we're finished
		if (StartFrameIndex + NumFrames >= NumFramesInWave)
		{
			int32 OutputFrameIndex = NumFramesInWave - StartFrameIndex - 1;
			if (bRequiresResampling)
			{
				OutputFrameIndex = RuntimeResampler.GetNumOutputFramesProducedByInputFrames(OutputFrameIndex);
			}
			OutputFrameIndex += BlockFrameIndex;
			OutEvents.Add(FSourceEvent(FSourceEvent::OnFinished, NumFramesInWave - 1, OutputFrameIndex));
		}

		// check if we'll finish in the next render, assuming the NumFrames we'll render next time wil be the same
		// note that we're using the actual number of requested input frames based on the Resampler
		// so in most cases, this should trigger exactly one block before "OnFinished" is triggered
		// In some cases, it can render a few blocks before "OnFinished" if the Frame Ratio changes between renders
		// It's also possible for this event to trigger in the same block as "OnFinished" in cases where the
		// Sound can be rendered in a single block, due to either a high speed setting or a tiny wave file
		if (!bHasNearlyFinished && StartFrameIndex + NumFrames * 2 >= NumFramesInWave)
		{
			// trigger "On nearly finished" at the beginning of this block
			OutEvents.Add(FSourceEvent(FSourceEvent::OnNearlyFinished, StartFrameIndex, BlockFrameIndex));

			// set this flag to avoid re-triggering in case the frame ratio changes between now and the next render
			bHasNearlyFinished = true;
		}

		return;
	}

	if (StartFrameIndex >= LoopEndFrameIndex)
	{
		StartFrameIndex = LoopStartFrameIndex;
		OutEvents.Add(FSourceEvent(FSourceEvent::Loop, LoopEndFrameIndex, BlockFrameIndex));
	}

	bool bLoopingThisBlock = (StartFrameIndex + NumFrames) > LoopEndFrameIndex;

	if (!bLoopingThisBlock)
	{
		GenerateSourceEventsInternal(StartFrameIndex, BlockFrameIndex, NumFrames, OutEvents);
		return;
	}

	int32 FirstNumFrames = LoopEndFrameIndex - StartFrameIndex;
	GenerateSourceEventsInternal(StartFrameIndex, BlockFrameIndex, FirstNumFrames, OutEvents);

	int32 OutputFrameIndex = LoopEndFrameIndex - StartFrameIndex - 1;
	if (bRequiresResampling)
	{
		OutputFrameIndex = RuntimeResampler.GetNumOutputFramesProducedByInputFrames(OutputFrameIndex);
	}
	OutputFrameIndex += BlockFrameIndex;
	OutEvents.Add(FSourceEvent(FSourceEvent::Loop, LoopEndFrameIndex, OutputFrameIndex));
	int32 SecondNumFrames = NumFrames - FirstNumFrames;
	GenerateSourceEventsInternal(LoopStartFrameIndex, OutputFrameIndex, SecondNumFrames, OutEvents);
}

void FSoundWaveProxyPlayer::GenerateSourceEventsInternal(int32 InSourceStartFrameIndex, int32 InOutputStartFrameIndex, int32 NumFrames, TArray<FSourceEvent>& OutEvents)
{
	if (SortedCuePoints.IsEmpty())
	{
		return;
	}
	
	int32 EndFrameIndex = InSourceStartFrameIndex + NumFrames;
	int32 CuePointIndex = Algo::LowerBoundBy(SortedCuePoints, InSourceStartFrameIndex, &FSoundWaveCuePoint::FramePosition);
	for ( ;SortedCuePoints.IsValidIndex(CuePointIndex); ++CuePointIndex)
	{
		int32 CuePointFrameIndex = SortedCuePoints[CuePointIndex].FramePosition;
		if (CuePointFrameIndex >= EndFrameIndex)
		{
			break;
		}

		int32 OutputFrameIndex = CuePointFrameIndex - InSourceStartFrameIndex;
		if (bRequiresResampling)
		{
			OutputFrameIndex = RuntimeResampler.GetNumOutputFramesProducedByInputFrames(OutputFrameIndex);
		}
		OutputFrameIndex += InOutputStartFrameIndex;
		OutEvents.Add(FSourceEvent(FSourceEvent::CuePoint, CuePointFrameIndex, OutputFrameIndex, CuePointIndex));
	}
}

void FSoundWaveProxyPlayer::RenderMultiChannelAudio(int32 BlockFrameIndex, Audio::FMultichannelBufferView& OutMultiChannelAudio, TArray<FSourceEvent>& OutEvents)
{
	const int32 NumFrames = Audio::GetMultichannelBufferNumFrames(OutMultiChannelAudio);
	GenerateSourceEvents(BlockFrameIndex, NumFrames, OutEvents);
	RenderMultiChannelAudio(OutMultiChannelAudio);
}

void FSoundWaveProxyPlayer::RenderMultiChannelAudio(const Audio::FMultichannelBufferView& OutMultiChannelAudio)
{
	check(SoundWaveDataPtr.IsValid());

	if (!ensure(Settings.OutputNumChannels == OutMultiChannelAudio.Num()))
	{
		return;
	}

	if (IsFinished())
	{
		for (const TArrayView<float>& Audio : OutMultiChannelAudio)
		{
			FMemory::Memset(Audio.GetData(), 0, Audio.Num() * Audio.GetTypeSize());
		}
		return;
	}
	
	const float FrameRatio = GetCurrentFrameRatio();

	const int32 OutputNumFrames = Audio::GetMultichannelBufferNumFrames(OutMultiChannelAudio);
	if (!bRequiresResampling && FMath::IsNearlyEqual(FrameRatio, 1.0f))
	{
		if (1 == NumChannels && NumChannels == Settings.OutputNumChannels)
		{
			GenerateSourceAudio(OutMultiChannelAudio[0]);
		}
		else
		{
			SourceAudio.SetNum(OutputNumFrames * NumChannels);
			GenerateSourceAudio(SourceAudio);
			ConvertDeinterleave->ProcessAudio(SourceAudio, OutMultiChannelAudio);
			SourceAudio.SetNum(0, EAllowShrinking::No);
		}
	}
	else
	{
		// once we've started resampling, we'll need to continue resampling to maintain continuity
		// even when the frame ratio ever goes back to 1
		bRequiresResampling = true;
		
		// resampling requires us to generate the source audio into a separate buffer
		RuntimeResampler.SetFrameRatio(FrameRatio);
		int32 SourceNumFrames = RuntimeResampler.GetNumInputFramesNeededToProduceOutputFrames(OutputNumFrames);

		// include the samples we used from the last render
		int32 LastNumSamples = SourceAudio.Num();
		SourceAudio.SetNum(SourceNumFrames * NumChannels);
		GenerateSourceAudio(MakeArrayView(SourceAudio.GetData() + LastNumSamples, SourceAudio.Num() - LastNumSamples));

		int32 NumFramesConsumed = 0;
		int32 NumFramesProduced = 0;
		if (1 == NumChannels && NumChannels == Settings.OutputNumChannels)
		{
			// Mono -> Mono
			// write the output of the resampler directly to the output
			RuntimeResampler.ProcessInterleaved(SourceAudio, OutMultiChannelAudio[0], NumFramesConsumed, NumFramesProduced);
		}
		else
		{
			// other sources require down/up mixing using the deinterleave converter
			ResampledAudio.SetNum(OutputNumFrames * NumChannels);
			RuntimeResampler.ProcessInterleaved(SourceAudio, ResampledAudio, NumFramesConsumed, NumFramesProduced);
			ConvertDeinterleave->ProcessAudio(ResampledAudio, OutMultiChannelAudio);
		}

		int32 NumSamplesConsumed = NumFramesConsumed * NumChannels;
		int32 NumSamplesRemaining = SourceAudio.Num() - NumSamplesConsumed;
		if (NumSamplesConsumed < SourceAudio.Num())
		{
			// move remaining samples to the start of the SourceBuffer for future processing
			check(NumSamplesRemaining % NumChannels == 0);
			FMemory::Memmove(SourceAudio.GetData(), SourceAudio.GetData() + NumSamplesConsumed, NumSamplesRemaining * sizeof(float));
		}
		SourceAudio.SetNum(NumSamplesRemaining, EAllowShrinking::No);
	}

	// Similar to Sequencer regularly re-queuing short chunks of the subtitles as it scrubs, this re-queues them from the sound/package names sent to the SubtitlesSubsystem.
	constexpr static float kSubtitleDisplayDuration = 0.125f;				// Subtitle display duration, in seconds
	constexpr static float kSubtitleRequeueDuration = 0.1f;					// How long in seconds to wait before queueing again, slightly shorter than the display duration.

	// Uses the first channel to count samples, so check that there's a channel there first.
	if (SoundWaveDataPtr->HasSubtitleAssetUserData() && NumChannels > 0)
	{
		Chan0SamplesSinceQueuedSubtitles += OutMultiChannelAudio[0].Num();
		check(Settings.OutputSampleRate > 0);

		const float ElapsedDuration = Chan0SamplesSinceQueuedSubtitles / Settings.OutputSampleRate;
		if (ElapsedDuration >= kSubtitleRequeueDuration)
		{
			Chan0SamplesSinceQueuedSubtitles = 0;

			// Send the sound's name to the subtitle system, where it will look up any associated subtitles and play them.
			// SubtitlesSubsystem will receive the delegate and send the FNames to the game thread for queueing.
			FSubtitlesAndClosedCaptionsDelegates::QueueSubtitleFromSoundWaveName.ExecuteIfBound(SoundWaveDataPtr->GetFName(), SoundWaveDataPtr->GetPackageName(), kSubtitleDisplayDuration);
		}
	}
}

int32 FSoundWaveProxyPlayer::GenerateSourceAudio(TArrayView<float> OutAudio)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FSoundWaveProxyPlayer::GenerateSourceAudio);

	check(SoundWaveDataPtr.IsValid());

	int32 NumFramesRequested = OutAudio.Num() / NumChannels;

	// check to see if the source audio is in sync 
	if (Settings.bMaintainAudioSync && AudioSyncFrameIndex != SourceFrameIndex)
	{
		// assert that our source frame is never ahead of the sync frame
		check(AudioSyncFrameIndex > SourceFrameIndex);
		// try to force the source audio to be in sync
		int32 NumSyncFrames = AudioSyncFrameIndex - SourceFrameIndex;
		int32 NumFramesSeeked = SimulateSeekForward(NumSyncFrames);
		if (NumSyncFrames == NumFramesSeeked)
		{
			// it should be in sync now
			check(AudioSyncFrameIndex == SourceFrameIndex);
		}
	}

	if (!bIsLooping)
	{
		int32 NumFramesGenerated = GenerateSourceAudioInternal(OutAudio);
		return NumFramesGenerated;
	}

	if (SourceFrameIndex >= LoopEndFrameIndex)
	{
		SeekToFrame(LoopStartFrameIndex);
	}

	bool bLoopingThisChunk = (SourceFrameIndex + NumFramesRequested) > LoopEndFrameIndex;

	if (!bLoopingThisChunk)
	{
		int32 NumFramesGenerated = GenerateSourceAudioInternal(OutAudio);
		return NumFramesGenerated;
	}
	
	int32 NumFramesRemaining = NumFramesRequested;
	int32 SampleOffset = 0;
	while (NumFramesRemaining > 0)
	{
		int32 NumFrames = FMath::Min(LoopEndFrameIndex - SourceFrameIndex, NumFramesRemaining);
		int32 NumSamples = NumFrames * NumChannels;
		GenerateSourceAudioInternal(OutAudio.Slice(SampleOffset, NumSamples));
		SampleOffset += NumSamples;
		NumFramesRemaining -= NumFrames;
		if (NumFramesRemaining > 0)
		{
			SeekToFrame(LoopStartFrameIndex);
		}
	}
	return NumFramesRequested;
}

int32 FSoundWaveProxyPlayer::GenerateSourceAudioInternal(TArrayView<float> OutAudio)
{
	check(CompressedAudioInfo.IsValid());
	check(DecodeBuffer.Num() > 0);
	check(OutAudio.Num() % NumChannels == 0);
	const int32 NumFramesTotal = FMath::Min(OutAudio.Num() / NumChannels, NumFramesInWave - SourceFrameIndex);
	int32 NumFramesRemaining = NumFramesTotal;
	int32 SampleOffset = 0;
	uint32 MaxBufferSize = DecodeBuffer.Num() * DecodeBuffer.GetTypeSize();
	uint32 DesiredBufferSize = OutAudio.Num() * DecodeBuffer.GetTypeSize();
	uint32 BufferSize = FMath::Min(DesiredBufferSize, MaxBufferSize);
	uint8* ByteBuffer = reinterpret_cast<uint8*>(DecodeBuffer.GetData());

	bool bFinished = false;
	while (!bFinished && NumFramesRemaining > 0)
	{
		int32 NumBytesStreamed = 0;
		if (SoundWaveDataPtr->IsStreaming())
		{
			bFinished = CompressedAudioInfo->StreamCompressedData(ByteBuffer, false, BufferSize, NumBytesStreamed);
		}
		else
		{
			bFinished = CompressedAudioInfo->ReadCompressedData(ByteBuffer, false, BufferSize);
			NumBytesStreamed = BufferSize;
		}

		if (NumBytesStreamed == 0)
		{
			break;
		}

		const int32 NumSamplesStreamed = NumBytesStreamed / sizeof(int16);
		const int32 NumFramesStreamed = NumSamplesStreamed / NumChannels;
		
		Audio::ArrayPcm16ToFloat(MakeArrayView(DecodeBuffer.GetData(), NumSamplesStreamed), MakeArrayView(OutAudio.GetData() + SampleOffset, NumSamplesStreamed));
		SampleOffset += NumSamplesStreamed;
		NumFramesRemaining -= NumFramesStreamed;
		DesiredBufferSize -= NumBytesStreamed;
		BufferSize = FMath::Min(DesiredBufferSize, MaxBufferSize);
	}

	// zero unset samples
	if (SampleOffset < OutAudio.Num())
	{
		FMemory::Memset(OutAudio.GetData() + SampleOffset, 0, (OutAudio.Num() - SampleOffset) * sizeof(float));
	}
	int32 NumFramesGenerated = NumFramesTotal - NumFramesRemaining;
	SourceFrameIndex = FMath::Min(SourceFrameIndex + NumFramesGenerated, NumFramesInWave);
	AudioSyncFrameIndex = FMath::Min(AudioSyncFrameIndex + NumFramesTotal, NumFramesInWave);
	return NumFramesGenerated;
}

int32 FSoundWaveProxyPlayer::SimulateSeekForward(int32 NumFrames)
{
	int32 NumFramesRemaining = NumFrames;
	int32 SampleOffset = 0;
	const uint32 MaxBufferSize = DecodeBuffer.Num() * DecodeBuffer.GetTypeSize();
	uint32 DesiredBufferSize = NumFrames * NumChannels * DecodeBuffer.GetTypeSize();
	uint32 BufferSize = FMath::Min(DesiredBufferSize, MaxBufferSize);
	uint8* ByteBuffer = (uint8*)DecodeBuffer.GetData();

	bool bFinished = false;
	while (!bFinished && NumFramesRemaining > 0)
	{
		int32 NumBytesStreamed = 0;
		if (SoundWaveDataPtr->IsStreaming())
		{
			bFinished = CompressedAudioInfo->StreamCompressedData(ByteBuffer, false, BufferSize, NumBytesStreamed);
		}
		else
		{
			bFinished = CompressedAudioInfo->ReadCompressedData(ByteBuffer, false, BufferSize);
			NumBytesStreamed = BufferSize;
		}

		if (NumBytesStreamed == 0)
		{
			break;
		}

		const int32 NumSamplesStreamed = NumBytesStreamed / sizeof(int16);
		const int32 NumFramesStreamed = NumSamplesStreamed / NumChannels;

		SampleOffset += NumSamplesStreamed;
		NumFramesRemaining -= NumFramesStreamed;
		DesiredBufferSize -= NumBytesStreamed;
		BufferSize = FMath::Min(DesiredBufferSize, MaxBufferSize);
	}

	int32 NumFramesGenerated = NumFrames - NumFramesRemaining;
	SourceFrameIndex = FMath::Min(SourceFrameIndex + NumFramesGenerated, NumFramesInWave);
	return NumFramesGenerated;
}

float FSoundWaveProxyPlayer::GetCurrentFrameRatio() const
{
	const float SampleRateFrameRatio = SampleRate / Settings.OutputSampleRate;
	return FMath::Clamp(SampleRateFrameRatio * Speed, Audio::FRuntimeResampler::MinFrameRatio, Audio::FRuntimeResampler::MaxFrameRatio);
}

void FSoundWaveProxyPlayer::SetSpeed(float InSpeed)
{
	ensureMsgf(Speed >= MinSpeed, TEXT("Speed is too slow! Clamping to MinSpeed = %.2f"), MinSpeed);
	ensureMsgf(Speed <= MaxSpeed, TEXT("Speed is too high! Clamping to MaxSpeed = %.2f"), MaxSpeed);
	Speed = FMath::Clamp(InSpeed, MinSpeed, MaxSpeed);
}

float FSoundWaveProxyPlayer::GetSpeed() const
{
	return Speed;	
}

bool FSoundWaveProxyPlayer::SeekToTime(float TimeSeconds)
{
	if (SampleRate <= 0 || NumFramesInWave < 0)
	{
		return false;
	}

	int32 InFrame = FMath::Clamp(static_cast<int32>(TimeSeconds * SampleRate), 0, NumFramesInWave);
	return SeekToFrame(InFrame);
}

void FSoundWaveProxyPlayer::SetLoop(bool InIsLooping, float InLoopStartTimeSeconds, float InLoopDurationSeconds)
{
	bIsLooping = InIsLooping;
	CurrentLoopStartTime = InLoopStartTimeSeconds;
	CurrentLoopDuration = InLoopDurationSeconds;

	if (!SoundWaveDataPtr)
	{
		return;
	}
	
	float LoopStartTimeSeconds = FMath::Clamp(CurrentLoopStartTime, 0.0f, DurationSeconds - MinLoopDurationSeconds);
	float LoopDurationSeconds = CurrentLoopDuration > 0.0f ? CurrentLoopDuration : DurationSeconds - CurrentLoopStartTime;
	LoopDurationSeconds = FMath::Clamp(LoopDurationSeconds, MinLoopDurationSeconds, DurationSeconds - CurrentLoopStartTime);

	LoopStartFrameIndex = FMath::Clamp(FMath::FloorToInt32(LoopStartTimeSeconds * SampleRate), 0, NumFramesInWave);
	LoopEndFrameIndex = FMath::Clamp(FMath::FloorToInt32(LoopDurationSeconds * SampleRate) + LoopStartFrameIndex, 0, NumFramesInWave);
	
	check(LoopStartFrameIndex >= 0);
	check(LoopEndFrameIndex > LoopStartFrameIndex);
	check(LoopEndFrameIndex <= NumFramesInWave);
}

bool FSoundWaveProxyPlayer::IsLooping() const
{
	if (!SoundWaveDataPtr)
	{
		return false;
	}
	return bIsLooping;
}

bool FSoundWaveProxyPlayer::IsFinished() const
{
	if (!SoundWaveDataPtr)
	{
		return false;
	}

	if (bIsLooping)
	{
		return false;
	}
	
	return SourceFrameIndex >= NumFramesInWave;
}

float FSoundWaveProxyPlayer::GetCurrentPlaybackTimeSeconds() const
{
	if (!SoundWaveDataPtr)
	{
		return 0.0f;
	}
	
	if (!ensure(SampleRate > 0.0f))
	{
		return 0.0f;
	}
	return SourceFrameIndex / SampleRate;	
}

int32 FSoundWaveProxyPlayer::GetCurrentPlaybackFrame() const
{
	if (!SoundWaveDataPtr)
	{
		return 0;
	}
	
	return SourceFrameIndex;
}


float FSoundWaveProxyPlayer::GetPlaybackProgress() const
{
	if (!SoundWaveDataPtr)
	{
		return 0.0f;
	}
	
	if (!ensure(NumFramesInWave > 0))
	{
		return 0.0f;
	}
	return SourceFrameIndex / static_cast<float>(NumFramesInWave);
}

float FSoundWaveProxyPlayer::GetLoopProgress() const
{
	if (!SoundWaveDataPtr)
	{
		return 0.0f;
	}
	
	if (!bIsLooping)
	{
		return 0.0f;
	}
	
	if (!ensure(NumFramesInWave > 0))
	{
		return 0.0f;
	}
	check(LoopStartFrameIndex >= 0);
	check(LoopEndFrameIndex > LoopStartFrameIndex);
	check(LoopEndFrameIndex <= NumFramesInWave);

	return (SourceFrameIndex - LoopStartFrameIndex) / static_cast<float>(LoopEndFrameIndex - LoopStartFrameIndex);
}

const TArray<FSoundWaveCuePoint>& FSoundWaveProxyPlayer::GetCuePoints() const
{
	return SortedCuePoints;
}

bool FSoundWaveProxyPlayer::SeekToFrame(int32 InFrame)
{
	if (!SoundWaveDataPtr)
	{
		UE_LOGF(LogAudio, Error, "FSoundWaveProxyPlayer::SeekToFrame, failed to seek - no Sound Wave set!");
		return false;
	}
	
	InFrame = FMath::Clamp(InFrame, 0, NumFramesInWave);
	
	if (!CompressedAudioInfo.IsValid())
	{
		// set the current frame index, but indicate that we've still failed
		SourceFrameIndex = InFrame;
		AudioSyncFrameIndex = InFrame;
		UE_LOGF(LogAudio, Verbose, "FSoundWaveProxyPlayer::SeekToFrame, failed to seek due to the decoder being invalid!: %ls", *SoundWaveDataPtr->GetFName().ToString());
		return false;
	}

	// ignore seek request if we're already at the specified time
	if (InFrame == SourceFrameIndex)
	{
		return true;
	}

	// reset the "NearlyFinished" flag
	bHasNearlyFinished = false;

	if (SoundWaveDataPtr->IsSeekable() && CompressedAudioInfo)
	{
		CompressedAudioInfo->SeekToFrame(InFrame);
		SourceFrameIndex = InFrame;
		AudioSyncFrameIndex = InFrame;
		return true;
	}

	FName Format = SoundWaveDataPtr->GetRuntimeFormat();
	UE_LOGF(LogAudio, Warning, "Attempt to seek on non-seekable wave: (format:%ls) for wave (package:%ls) to frame '%d'",
		*Format.ToString(),
		*SoundWaveDataPtr->GetPackageName().ToString(),
		InFrame);
	
	// For non-seekable streaming waves, use a fallback method to seek
	IConsoleVariable* SimulateSeekCVar = IConsoleManager::Get().FindConsoleVariable(TEXT("au.SoundWaveProxyReader.SimulateSeek"));
	const bool bSimulateSeek = SimulateSeekCVar ? SimulateSeekCVar->GetInt() == 1 : false;
	if (bSimulateSeek)
	{
		UE_CLOG(bFallbackSeekMethodWarningLogged == false, LogAudio, Warning, TEXT("Simulating seeking in wave which is not seekable (package:%s). For better performance, set wave to a seekable format"), *SoundWaveDataPtr->GetPackageName().ToString());
		bFallbackSeekMethodWarningLogged = true;

		if (InFrame < SourceFrameIndex)
		{
			// if we're seeking backwards, we need to reset the decoder
			if (!InitializeDecoder(Settings.MaxDecodeSizeFrames))
			{
				SourceFrameIndex = InFrame;
				AudioSyncFrameIndex = InFrame;
				UE_LOGF(LogAudio, Verbose, "FSoundWaveProxyPlayer::SeekToFrame, failed to seek due to the decoder being invalid!: %ls", *SoundWaveDataPtr->GetFName().ToString());
				return false;
			}
			SourceFrameIndex = 0;
		}

		int32 SeekNumFrames = InFrame - SourceFrameIndex;
		int32 NumFramesGenerated = SimulateSeekForward(SeekNumFrames);
		SourceFrameIndex = FMath::Min(SourceFrameIndex + NumFramesGenerated, NumFramesInWave);
		AudioSyncFrameIndex = InFrame;
		return NumFramesGenerated == SeekNumFrames;
	}

	return false;
}


bool FSoundWaveProxyPlayer::InitializeDecoder(int32 MaxDecodeSizeInFrames)
{
	using namespace Audio;

	TRACE_CPUPROFILER_EVENT_SCOPE(FSoundWaveProxyPlayer::InitializeDecoder);

	const TSharedRef<const FSoundWaveData> SoundWaveDataRef = SoundWaveDataPtr.ToSharedRef();
	FName Format = SoundWaveDataRef->GetRuntimeFormat();
	IAudioInfoFactory* Factory = IAudioInfoFactoryRegistry::Get().Find(Format);
	if (!ensure(Factory))
	{
		UE_LOG(LogAudio, Error, TEXT("FSoundWaveProxyPlayer::InitializeDecoder: Failed to create CompressedAudioInfo for wave (package: %s). Unable to find AudioInfoFactory for format: %s"),
			*SoundWaveDataRef->GetPackageName().ToString(), *Format.ToString());
		return false;
	}

	// Fast path: try the decode cache before falling back to the factory. The cache
	// returns nullptr if disabled or if the wave duration exceeds the auto-cache
	// threshold, in which case we just construct a fresh decoder from the factory.
	TUniquePtr<ICompressedAudioInfo> InfoInstance;
	if (SoundWaveProxyDecodeCache::IsEnabled())
	{
		InfoInstance = SoundWaveProxyDecodeCache::CreateDecoderInstance(SoundWaveDataRef);
	}

	if (!InfoInstance.IsValid())
	{
		InfoInstance.Reset(IAudioInfoFactoryRegistry::Get().Create(Format, SoundWaveDataRef->GetFName()));
	}

	if (!ensure(InfoInstance.IsValid()))
	{
		UE_LOG(LogAudio, Error, TEXT("FSoundWaveProxyPlayer::InitializeDecoder: Failed to create CompressedAudioInfo for wave (package: %s). Unable to create info from factory for for format: %s"), *SoundWaveDataRef->GetPackageName().ToString(), *Format.ToString());
		return false;
	}

	FSoundQualityInfo Info;
	if (SoundWaveDataPtr->IsStreaming())
	{
		if (!InfoInstance->StreamCompressedInfo(SoundWaveDataPtr.ToSharedRef(), &Info))
		{
			UE_LOG(LogAudio, Error, TEXT("FSoundWaveProxyPlayer::InitializeDecoder: Failed to create CompressedAudioInfo for wave (package: %s). Unable to stream compressed info for streaming wave"), *SoundWaveDataPtr->GetPackageName().ToString());
			return false;
		}
	}
	else
	{
		if (!InfoInstance->ReadCompressedInfo(SoundWaveDataPtr->GetResourceData(), SoundWaveDataPtr->GetResourceSize(), &Info))
		{
			UE_LOG(LogAudio, Error, TEXT("FSoundWaveProxyPlayer::InitializeDecoder: Failed to create CompressedAudioInfo for wave (package: %s). Unable to read compressed info for non-streaming wave"), *SoundWaveDataPtr->GetPackageName().ToString());
			return false;
		}
	}

	SampleRate = SoundWaveDataPtr->GetSampleRate();
	NumChannels = SoundWaveDataPtr->GetNumChannels();
	NumFramesInWave = SoundWaveDataPtr->GetNumFrames();
	DurationSeconds = SoundWaveDataPtr->GetDuration();

	CompressedAudioInfo.Reset(InfoInstance.Release());

	// BEGIN HACK
	// Read the sample rate and number of frames from the header 
	// Similar to refreshing the wave data in FMixerBuffer::CreateStreamingBuffer
	// This is a runtime hack to address incorrect sample rate on soundwaves 
	// on platforms with Resample for Device enabled (UE-183237)
	SampleRate = Info.SampleRate;
	uint32 NumFrames = (uint32)((float)Info.Duration * Info.SampleRate);
	if (NumFrames > 0)
	{
		NumFramesInWave = NumFrames;
	}
	// END HACK

	// initialized decode buffers
	const int32 DecodeSize = ICompressedAudioInfo::ConformDecodeSize(MaxDecodeSizeInFrames);
	check(DecodeSize > 0);
	DecodeBuffer.SetNum(DecodeSize * NumChannels, EAllowShrinking::Yes);
	return true;
}

void FSoundWaveProxyPlayer::InitializeResampler(int32 NumInputChannels)
{
	bRequiresResampling = false;
	RuntimeResampler.Reset(NumInputChannels);
	ResampledAudio.Reset();
}
