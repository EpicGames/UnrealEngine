// Copyright Epic Games, Inc. All Rights Reserved.

#include "Sound/SoundWaveProxyReader.h"

#include "Audio.h"
#include "Audio/AudioDebug.h"
#include "Audio/AudioTimingLog.h"
#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "DSP/BufferVectorOperations.h"
#include "DSP/Dsp.h"
#include "DSP/FloatArrayMath.h"
#include "HAL/IConsoleManager.h"
#include "HAL/LowLevelMemTracker.h"
#include "HAL/Platform.h"
#include "HAL/UnrealMemory.h"
#include "Interfaces/IAudioFormat.h"
#include "Math/UnrealMathUtility.h"
#include "Misc/AssertionMacros.h"
#include "ProfilingDebugging/CpuProfilerTrace.h"
#include "Sound/SoundWave.h"
#include "Templates/SharedPointer.h"
#include "Templates/UniquePtr.h"

static int32 GSoundWaveProxyReaderSimulateSeekOnNonSeekable = 0;
FAutoConsoleVariableRef CVarSoundWaveProxyReaderSimulateSeekOnNonSeekable(
	TEXT("au.SoundWaveProxyReader.SimulateSeek"),
	GSoundWaveProxyReaderSimulateSeekOnNonSeekable,
	TEXT("If true, SoundWaves which are not of a seekable format will simulate seek calls by reading and discarding samples.\n")
	TEXT("0: Do not simulate seek, !0: Simulate seek"),
	ECVF_Default);

PRAGMA_DISABLE_DEPRECATION_WARNINGS

uint32 FSoundWaveProxyReader::ConformDecodeSize(uint32 InMaxDesiredDecodeSizeInFrames)
{
	static_assert(DefaultMinDecodeSizeInFrames >= DecodeSizeQuantizationInFrames, "Min decode size less than decode size quantization");
	static_assert((DefaultMinDecodeSizeInFrames % DecodeSizeQuantizationInFrames) == 0, "Min decode size must be equally divisible by decode size quantization");

	const uint32 QuantizedDecodeSizeInFrames = (InMaxDesiredDecodeSizeInFrames / DecodeSizeQuantizationInFrames) * DecodeSizeQuantizationInFrames;
	const uint32 ConformedDecodeSizeInFrames = FMath::Max(QuantizedDecodeSizeInFrames, DefaultMinDecodeSizeInFrames);

	check((QuantizedDecodeSizeInFrames > 0) && (ConformedDecodeSizeInFrames % QuantizedDecodeSizeInFrames) == 0);
	check(ConformedDecodeSizeInFrames > 0);

	return ConformedDecodeSizeInFrames;
}

/** Construct a wave proxy reader.
 *
 * @param InWaveData - A TSharedRef of a FSoundWaveData which is to be played.
 * @param InSettings - Reader settings.
 */
FSoundWaveProxyReader::FSoundWaveProxyReader(FSoundWaveDataRef InWaveData, const FSettings& InSettings)
	: WaveDataRef(InWaveData)
	, Settings(InSettings)
{
	// Get local copies of some values from the proxy. 
	SampleRate = WaveDataRef->GetSampleRate();
	NumChannels = WaveDataRef->GetNumChannels();
	NumFramesInWave = WaveDataRef->GetNumFrames();
	DurationInSeconds = WaveDataRef->GetDuration();
	MaxLoopStartTimeInSeconds = FMath::Max(0, DurationInSeconds - MinLoopDurationInSeconds);

	UE_CLOGF(Audio::MatchesLogFilter(WaveDataRef->GetFName()),LogAudioTiming, Verbose,
		"FSoundWaveProxyReader ctor: Name=%ls,NumChannels=%d, Rate=%.2f, NumFrames=%d", *InWaveData->GetFName().ToString(), NumChannels, SampleRate, NumFramesInWave);

	// Clamp times
	Settings.StartTimeInSeconds = FMath::Clamp(Settings.StartTimeInSeconds, 0.f, DurationInSeconds);
	Settings.LoopStartTimeInSeconds = ClampLoopStartTime(Settings.LoopStartTimeInSeconds);
	Settings.LoopDurationInSeconds = ClampLoopDuration(Settings.LoopDurationInSeconds);

	// Setup frame indices
	CurrentFrameIndex = 0;
	UpdateLoopBoundaries();

	// Determine max size of decode buffer
	Settings.MaxDecodeSizeInFrames = FMath::Max(DefaultMinDecodeSizeInFrames, Settings.MaxDecodeSizeInFrames);

	// Prepare to read audio
	bIsDecoderValid = InitializeDecoder(Settings.StartTimeInSeconds);
	if (!bIsDecoderValid)
	{
		UE_LOGF(LogAudio, Warning, "Failed to InitializeDecoder in FSoundWaveProxyReader(), Package: %ls", *WriteToString<64>(InWaveData->GetPackageName()));
	}

	// set the decoder to "Fail" if we're unable to create a decoder
	DecodeResult = bIsDecoderValid ? DecodeResult : EDecodeResult::Fail;
}

/** Create a wave proxy reader.
 *
 * @param InWaveProxy - A TSharedRef of a FSoundWaveProxy which is to be played.
 * @param InSettings - Reader settings.
 */
TUniquePtr<FSoundWaveProxyReader> FSoundWaveProxyReader::Create(FSoundWaveDataRef InWaveDataRef, const FSettings& InSettings)
{
	if (InWaveDataRef->GetSampleRate() <= 0.f)
	{
		UE_LOG(LogAudio, Warning, TEXT("Cannot create FSoundWaveProxyReader due to invalid sample rate (%f). Package: %s"), InWaveDataRef->GetSampleRate(), *InWaveDataRef->GetPackageName().ToString());
		return TUniquePtr<FSoundWaveProxyReader>();
	}

	if (InWaveDataRef->GetNumChannels() <= 0)
	{
		UE_LOG(LogAudio, Warning, TEXT("Cannot create FSoundWaveProxyReader due to invalid num channels (%d). Package: %s"), InWaveDataRef->GetNumChannels(), *InWaveDataRef->GetPackageName().ToString());
		return TUniquePtr<FSoundWaveProxyReader>();
	}

	if (InWaveDataRef->GetNumFrames() <= 0)
	{
		UE_LOG(LogAudio, Warning, TEXT("Cannot create FSoundWaveProxyReader due to invalid num frames (%d). Package: %s"), InWaveDataRef->GetNumFrames(), *InWaveDataRef->GetPackageName().ToString());
		return TUniquePtr<FSoundWaveProxyReader>();
	}

	return TUniquePtr<FSoundWaveProxyReader>(new FSoundWaveProxyReader(InWaveDataRef, InSettings));
}

TUniquePtr<FSoundWaveProxyReader> FSoundWaveProxyReader::Create(FSoundWaveProxyRef InWaveProxy, const FSettings& InSettings)
{
	return FSoundWaveProxyReader::Create(InWaveProxy->GetSoundWaveDataRef(), InSettings);
}


/** Set whether the reader should loop the audio or not. */
void FSoundWaveProxyReader::SetIsLooping(bool bInIsLooping)
{
	if (Settings.bIsLooping != bInIsLooping)
	{
		Settings.bIsLooping = bInIsLooping;
		UpdateLoopBoundaries();
	}
}

/** Sets the beginning position of the loop. */
void FSoundWaveProxyReader::SetLoopStartTime(float InLoopStartTimeInSeconds)
{
	InLoopStartTimeInSeconds = ClampLoopStartTime(InLoopStartTimeInSeconds);
	if (!FMath::IsNearlyEqual(Settings.LoopStartTimeInSeconds, InLoopStartTimeInSeconds))
	{
		Settings.LoopStartTimeInSeconds = InLoopStartTimeInSeconds;
		UpdateLoopBoundaries();
	}
}

/** Sets the duration of the loop in seconds. If the value is negative, the
 * loop duration consists of the entire file. */
void FSoundWaveProxyReader::SetLoopDuration(float InLoopDurationInSeconds)
{
	InLoopDurationInSeconds = ClampLoopDuration(InLoopDurationInSeconds);
	if (!FMath::IsNearlyEqual(Settings.LoopDurationInSeconds, InLoopDurationInSeconds))
	{
		Settings.LoopDurationInSeconds = InLoopDurationInSeconds;
		UpdateLoopBoundaries();
	}
}

bool FSoundWaveProxyReader::SeekToTime(float InSeconds)
{
	int32 InFrameIndex = FMath::Clamp(static_cast<int32>(InSeconds * GetSampleRate()), 0, GetNumFramesInWave());

	if (!bIsDecoderValid)
	{
		// set the current frame index, but indicate that we've still failed
		CurrentFrameIndex = InFrameIndex;
		UE_LOGF(LogAudio, Verbose, "FSoundWaveProxyReader::SeekToTime, failed to seek due to the decoder being invalid!: %ls", *WaveDataRef->GetFName().ToString());
		return false;
	}
	
	// ignore seek request if we're already at the specified time
	if (InFrameIndex == CurrentFrameIndex)
	{
		return true;
	}

	UE_CLOGF(Audio::MatchesLogFilter(WaveDataRef->GetFName()), LogAudioTiming, Verbose,
		"FSoundWaveProxyReader SeekToTime: SeekTime=%.2f, Name=%ls, NumChannels=%d, Rate=%.2f, NumFrames=%d",
		InSeconds, *WaveDataRef->GetFName().ToString(), NumChannels, SampleRate, NumFramesInWave);
	
	if (WaveDataRef->IsSeekable() && CompressedAudioInfo)
	{
		CompressedAudioInfo->SeekToFrame(InFrameIndex);
		CurrentFrameIndex = InFrameIndex;
		DecoderOutput.SetNum(0);
		NumDecodeSamplesToDiscard = 0;
		DecodeResult = EDecodeResult::MoreDataRemaining;
		return true;
	}
	// Direct seeking is not supported. A new decoder must be created. 
	bIsDecoderValid = InitializeDecoder(InSeconds);
	return bIsDecoderValid;
}

bool FSoundWaveProxyReader::CanProduceMoreAudio() const
{
	const bool bDecoderOutputHasMoreData = DecoderOutput.Num() > 0;
	const bool bDecoderCanDecodeMoreData = bIsDecoderValid && (EDecodeResult::MoreDataRemaining == DecodeResult);
	return bDecoderOutputHasMoreData || bDecoderCanDecodeMoreData;
}

bool FSoundWaveProxyReader::SeekToFrame(uint32 InFrameNum)
{
	if (!bIsDecoderValid)
	{
		// set the current frame index, but indicate that we've still failed
		CurrentFrameIndex = InFrameNum;
		UE_LOGF(LogAudio, Verbose, "FSoundWaveProxyReader::SeekToTime, failed to seek due to the decoder being invalid!");
		return false;
	}
	
	// ignore seek request if we're already at the specified time
	if (InFrameNum == CurrentFrameIndex)
	{
		return true;
	}

	UE_CLOGF(Audio::MatchesLogFilter(WaveDataRef->GetFName()), LogAudioTiming, Verbose,
		"FSoundWaveProxyReader SeekToFrame: SeekFrame=%u, Name=%ls, NumChannels=%d, Rate=%.2f, NumFrames=%d",
		InFrameNum, *WaveDataRef->GetFName().ToString(), NumChannels, SampleRate, NumFramesInWave);

	if (WaveDataRef->IsSeekable() && CompressedAudioInfo)
	{
		CompressedAudioInfo->SeekToFrame(InFrameNum);
		CurrentFrameIndex = InFrameNum;
		DecoderOutput.SetNum(0);
		NumDecodeSamplesToDiscard = 0;
		DecodeResult = EDecodeResult::MoreDataRemaining;
		return true;
	}

	// Direct seeking is not supported. A new decoder must be created.
	float Seconds = static_cast<float>(InFrameNum) / FMath::Max(1.f, GetSampleRate());
	bIsDecoderValid = InitializeDecoder(Seconds);
	return bIsDecoderValid;
}

/** Copies audio into OutBuffer. It returns the number of samples copied.
 * Samples not written to will be set to zero.
 */
int32 FSoundWaveProxyReader::PopAudio(Audio::FAlignedFloatBuffer& OutBuffer)
{
	using namespace Audio;

	TRACE_CPUPROFILER_EVENT_SCOPE(FSoundWaveProxyReader::PopAudio);

	if ((0 == NumChannels) || !bIsDecoderValid)
	{
		if (OutBuffer.Num() > 0)
		{
			FMemory::Memset(OutBuffer.GetData(), 0, sizeof(float) * OutBuffer.Num());
		}
		return 0;
	}

	checkf(0 == (OutBuffer.Num() % NumChannels), TEXT("Output buffer size must be evenly divisible by the number of channels"));

	TArrayView<float> OutBufferView{OutBuffer};
	int32 NumSamplesUnset = OutBuffer.Num();
	int32 NumSamplesCopied = 0;

	bool bDecoderOutputHasMoreData = DecoderOutput.Num() > 0;
	bool bDecoderCanDecodeMoreData = bIsDecoderValid && (EDecodeResult::MoreDataRemaining == DecodeResult);
	bool bCanProduceMoreAudio = bDecoderOutputHasMoreData || bDecoderCanDecodeMoreData;

	while ((NumSamplesUnset > 0) && bCanProduceMoreAudio)
	{
		int32 NumSamplesCopiedThisLoop = PopAudioFromDecoderOutput(OutBufferView);

		// Update sample counters and output buffer view
		NumSamplesCopied += NumSamplesCopiedThisLoop;
		NumSamplesUnset -= NumSamplesCopiedThisLoop;
		OutBufferView = OutBufferView.Slice(NumSamplesCopiedThisLoop, NumSamplesUnset);

		UE_CLOGF(Audio::MatchesLogFilter(WaveDataRef->GetFName()), LogAudioTiming, VeryVerbose,
				"FSoundWaveProxyReader PopAudio, NumSamplesCopiedThisLoop=%d, NumSamplesCopied=%d, NumSamplesUnset=%d, Name=%ls, NumChannels=%d, Rate=%.2f, NumFrames=%d",
						NumSamplesCopiedThisLoop, NumSamplesCopied, NumSamplesUnset,  *WaveDataRef->GetFName().ToString(), NumChannels, SampleRate, NumFramesInWave)
		
		if (Settings.bIsLooping)
		{
			// Seek to loop start if we have used up all our decodable samples or 
			// are at the loop boundary.
			if (0 == DecoderOutput.Num())
			{
				if ((!bDecoderCanDecodeMoreData) || (CurrentFrameIndex >= LoopEndFrameIndex))
				{
					UE_CLOGF(Audio::MatchesLogFilter(WaveDataRef->GetFName()), LogAudioTiming, Verbose,
								"FSoundWaveProxyReader PopAudio (looping-return to start) (((!bDecoderCanDecodeMoreData) || (CurrentFrameIndex >= LoopEndFrameIndex))), Name=%ls, NumChannels=%d, Rate=%.2f, NumFrames=%d",
									*WaveDataRef->GetFName().ToString(), NumChannels, SampleRate, NumFramesInWave)
					
					SeekToTime(Settings.LoopStartTimeInSeconds);
					bDecoderCanDecodeMoreData = bIsDecoderValid && (EDecodeResult::Fail != DecodeResult);
				}
			}
		}

		// Determine if we can / should decode more data. 
		if ((NumSamplesUnset > 0) && (DecoderOutput.Num() == 0) && bDecoderCanDecodeMoreData)
		{
			DecodeResult = Decode();
			bDecoderCanDecodeMoreData = EDecodeResult::MoreDataRemaining == DecodeResult;
		}

		bDecoderOutputHasMoreData = DecoderOutput.Num() > 0;
		bCanProduceMoreAudio = bDecoderOutputHasMoreData || bDecoderCanDecodeMoreData;

		if (bCanProduceMoreAudio && DecoderOutput.Num() == 0)
		{
			UE_CLOGF(Audio::MatchesLogFilter(WaveDataRef->GetFName()),LogAudioTiming, Verbose,
				"FSoundWaveProxyReader PopAudio (break-loop, (bCanProduceMoreAudio && DecoderOutput.Num() == 0), Name=%ls, NumChannels=%d, Rate=%.2f, NumFrames=%d",
					*WaveDataRef->GetFName().ToString(), NumChannels, SampleRate, NumFramesInWave);
			
			// we can produce more audio, but we were unable to
			// this is likely due to the streaming data not being available yet
			// let's early out to avoid a hitch and hope that it's ready on the next read
			break;
		}
	}

	UE_CLOGF(Audio::MatchesLogFilter(WaveDataRef->GetFName()), LogAudioTiming, Verbose,
		"FSoundWaveProxyReader PopAudio (loop-end), OutBufferView.Num()=%d, NumSamplesCopied=%d, Name=%ls, NumChannels=%d, Rate=%.2f, NumFrames=%d",
		OutBufferView.Num(), NumSamplesCopied,  *WaveDataRef->GetFName().ToString(), NumChannels, SampleRate, NumFramesInWave);
	
	// Zero pad any unset samples. 
	if (OutBufferView.Num() > 0)
	{
		// Zero out audio that was not set.
		FMemory::Memset(OutBufferView.GetData(), 0, sizeof(float) * OutBufferView.Num());

		if (Settings.bMaintainAudioSync)
		{
			ensureMsgf(!Settings.bIsLooping, TEXT("Currently can't BOTH loop a wave and have it maintain sync. The code that does the bookkeeping when the decoder underruns is not robust enough to handle that situation."));

			// if we were asked to maintain audio sync, then do some extra book keeping
			// keep track of the number of samples we'll need to discard the next time we try to read from the decoder
			NumDecodeSamplesToDiscard += OutBufferView.Num();
			// and pretend we are advancing through the data even though we aren't...
			CurrentFrameIndex += OutBufferView.Num() / NumChannels;
			if (CurrentFrameIndex > GetNumFramesInWave())
			{
				// but don't pretend to go passed the end!...
				CurrentFrameIndex = GetNumFramesInWave();
			}
			// Note: We can do the above because later we will NOT advance CurrentFrameIndex for the decoded samples that we discard!)
		}
	}

	return NumSamplesCopied;
}

int32 FSoundWaveProxyReader::PopAudioFromDecoderOutput(TArrayView<float> OutBufferView)
{
	check(NumChannels > 0);

	int32 NumDiscarded = DecoderOutput.Pop(NumDecodeSamplesToDiscard);
	NumDecodeSamplesToDiscard = NumDecodeSamplesToDiscard - NumDiscarded;
	check(NumDecodeSamplesToDiscard >= 0);
	
	int32 NumSamplesCopied = 0;
	if (DecoderOutput.Num() > 0)
	{
		// Get samples from the decoder buffer.
		NumSamplesCopied = DecoderOutput.Pop(OutBufferView.GetData(), OutBufferView.Num());
		
		int32 NumFramesCopied = NumSamplesCopied / NumChannels;
		CurrentFrameIndex += NumFramesCopied;

		// Check whether the samples copied from the decoder extend 
		// past the end of the loop.
		if (Settings.bIsLooping)
		{
			const bool bDidOvershoot = CurrentFrameIndex >= LoopEndFrameIndex;
			if (bDidOvershoot)
			{
				// Rewind sample counters if the loop boundary was overshot.
				NumFramesCopied -= (CurrentFrameIndex - LoopEndFrameIndex);
				CurrentFrameIndex = LoopEndFrameIndex;
				// If Settings.bIsLooping info was altered, NumFramesCopied can end up 
				// negative if the current frame index is past the loop end frame. 
				NumFramesCopied = FMath::Max(0, NumFramesCopied);
				NumSamplesCopied = NumFramesCopied * NumChannels;

				// Remove any remaining samples in the decoder because they 
				// are past the end of the loop.
				DecoderOutput.SetNum(0);
			}

		}
	}

	return NumSamplesCopied;
}

bool FSoundWaveProxyReader::InitializeDecoder(float InStartTimeInSeconds)
{
	using namespace Audio;

	TRACE_CPUPROFILER_EVENT_SCOPE(FSoundWaveProxyReader::InitializeDecoder);

	FName Format = WaveDataRef->GetRuntimeFormat();
	TUniquePtr<ICompressedAudioInfo> InfoInstance;
	InfoInstance.Reset(IAudioInfoFactoryRegistry::Get().Create(Format, WaveDataRef->GetFName()));

	if (!ensure(InfoInstance.IsValid()))
	{
		UE_LOGF(LogAudio, Error, "FSoundWaveProxyReader::InitializeDecoder: Failed to create CompressedAudioInfo for wave (package: %ls). Unable to create info from factory for format: %ls", *WaveDataRef->GetPackageName().ToString(), *Format.ToString());
		return false;
	}

	FSoundQualityInfo Info;
	if (WaveDataRef->IsStreaming())
	{
		if (!InfoInstance->StreamCompressedInfo(WaveDataRef, &Info))
		{
			UE_LOGF(LogAudio, Error, "FSoundWaveProxyReader::InitializeDecoder: Failed to create CompressedAudioInfo for wave (package: %ls). Unable to stream compressed info for streaming wave", *WaveDataRef->GetPackageName().ToString());
			return false;
		}
	}
	else
	{
		if (!InfoInstance->ReadCompressedInfo(WaveDataRef->GetResourceData(), WaveDataRef->GetResourceSize(), &Info))
		{
			UE_LOGF(LogAudio, Error, "FSoundWaveProxyReader::InitializeDecoder: Failed to create CompressedAudioInfo for wave (package: %ls). Unable to read compressed info for non-streaming wave", *WaveDataRef->GetPackageName().ToString());
			return false;
		}
	}

	CompressedAudioInfo.Reset(InfoInstance.Release());

	// Read the sample rate and number of frames from the header 
	// Similar to refreshing the wave data in FMixerBuffer::CreateStreamingBuffer
	// This is a runtime hack to address incorrect sample rate on soundwaves 
	// on platforms with Resample for Device enabled (UE-183237)
	SampleRate = Info.SampleRate;
	uint32 NumFrames = (uint32)((float)Info.Duration * Info.SampleRate);
	if (NumFrames > 0)
	{
		NumFramesInWave = NumFrames;
	}
	// end hack

	// Seek input to start time.
	if (!FMath::IsNearlyEqual(0.0f, InStartTimeInSeconds))
	{
		if (WaveDataRef->IsSeekable())
		{
			CompressedAudioInfo->SeekToTime(InStartTimeInSeconds);
		}
		else
		{
			UE_LOGF(LogAudio, Warning, "Attempt to seek on non-seekable wave: (format:%ls) for wave (package:%ls) to time '%.6f'",
				*Format.ToString(),
				*WaveDataRef->GetPackageName().ToString(),
				InStartTimeInSeconds);
		}
	}

	CurrentFrameIndex = FMath::Clamp(static_cast<int32>(InStartTimeInSeconds * GetSampleRate()), 0, GetNumFramesInWave());

	// initialized decode buffers
	const uint32 DecodeSize = ConformDecodeSize(Settings.MaxDecodeSizeInFrames);
	NumFramesPerDecode = FMath::Max(DecodeSize, NumFramesPerDecode);
	ResidualBuffer.SetNum(NumFramesPerDecode * NumChannels);
	SampleConversionBuffer.SetNumUninitialized(NumFramesPerDecode * NumChannels);
	DecoderOutput.Reserve(/* MinCapacity = */ NumFramesPerDecode * NumChannels * 2, /* bRetainExistingSamples = */ false);
	NumDecodeSamplesToDiscard = 0;

	// For non-seekable streaming waves, use a fallback method to seek
	// to the start time. 
	const bool bUseFallbackSeekMethod = (GSoundWaveProxyReaderSimulateSeekOnNonSeekable != 0) && (InStartTimeInSeconds != 0.f) && (!WaveDataRef->IsSeekable());
	if (bUseFallbackSeekMethod)
	{
		if (!bFallbackSeekMethodWarningLogged)
		{
			UE_LOGF(LogAudio, Warning, "Simulating seeking in wave which is not seekable (package:%ls). For better performance, set wave to a seekable format", *WaveDataRef->GetPackageName().ToString());
			bFallbackSeekMethodWarningLogged = true;
		}

		int32 NumSamplesToDiscard = CurrentFrameIndex * NumChannels;
		DiscardSamples(NumSamplesToDiscard);
	}

	if (!CompressedAudioInfo.IsValid())
	{
		UE_LOGF(LogAudio, Error, "Failed to create decoder (format:%ls) for wave (package:%ls)", *Format.ToString(), *WaveDataRef->GetPackageName().ToString());
		DecodeResult = EDecodeResult::Fail;
		return false;
	}
	else
	{
		// The DecodeResult needs to be set to a valid state in case the prior
		// decoder finished or failed. 
		DecodeResult = EDecodeResult::MoreDataRemaining;
	}

	return true;
}

int32 FSoundWaveProxyReader::DiscardSamples(int32 InNumSamplesToDiscard)
{
	int32 NumSamplesDiscarded = 0;
	while (InNumSamplesToDiscard > 0)
	{
		DecodeResult = Decode();

		int32 NumSamplesToDiscardThisLoop = FMath::Min((int32)DecoderOutput.Num(), InNumSamplesToDiscard);

		if (NumSamplesToDiscardThisLoop < 1)
		{
			UE_LOGF(LogAudio, Error, "Failed to decode samples (package: %ls)", *WaveDataRef->GetPackageName().ToString());
			break;
		}

		int32 ActualNumSamplesDiscarded = DecoderOutput.Pop(NumSamplesToDiscardThisLoop);
		InNumSamplesToDiscard -= ActualNumSamplesDiscarded;
		NumSamplesDiscarded += ActualNumSamplesDiscarded;

		if ((InNumSamplesToDiscard > 0) && (DecodeResult != EDecodeResult::MoreDataRemaining))
		{
			UE_LOGF(LogAudio, Error, "Failed to simulate seek (seek to frame: %d, package: %ls)", CurrentFrameIndex, *WaveDataRef->GetPackageName().ToString());
			break;
		}
	}

	return NumSamplesDiscarded;
}

float FSoundWaveProxyReader::ClampLoopStartTime(float InStartTimeInSeconds)
{
	return FMath::Clamp(InStartTimeInSeconds, 0.f, MaxLoopStartTimeInSeconds);
}

float FSoundWaveProxyReader::ClampLoopDuration(float InDurationInSeconds)
{
	if (InDurationInSeconds <= 0.f)
	{
		return MaxLoopDurationInSeconds;
	}
	return FMath::Max(InDurationInSeconds, MinLoopDurationInSeconds);
}

void FSoundWaveProxyReader::UpdateLoopBoundaries()
{
	if (Settings.bIsLooping)
	{
		LoopStartFrameIndex = FMath::FloorToInt(Settings.LoopStartTimeInSeconds * SampleRate);
		const int32 MinLoopDurationInFrames = FMath::CeilToInt(MinLoopDurationInSeconds * SampleRate);
		const float LoopEndTime = Settings.LoopStartTimeInSeconds + Settings.LoopDurationInSeconds;
		const int32 MinLoopEndFrameIndex = LoopStartFrameIndex + MinLoopDurationInFrames;
		const int32 MaxLoopEndFrameIndex = NumFramesInWave;

		LoopEndFrameIndex = FMath::Clamp(LoopEndTime * SampleRate, MinLoopEndFrameIndex, MaxLoopEndFrameIndex);
	}
	else
	{
		LoopStartFrameIndex = 0;
		LoopEndFrameIndex = NumFramesInWave;
	}
}

FSoundWaveProxyReader::EDecodeResult FSoundWaveProxyReader::Decode()
{
	check(CompressedAudioInfo.IsValid());

	bool bFinished = false;

	int32 NumFramesRemaining = NumFramesPerDecode;
	uint32 BuffSizeInBytes = ResidualBuffer.Num() * sizeof(int16);
	uint32 BuffSizeInFrames = NumFramesPerDecode;
	uint8* Buff = (uint8*)ResidualBuffer.GetData();

	// cache the streaming flag off the wave
	// if it has changed since the last Decode() call, bail
	// something has probably changed in editor
	if (bIsFirstDecode)
	{
		bIsFirstDecode = false;
	}
	else
	{
		if (bPreviousIsStreaming != WaveDataRef->IsStreaming())
		{
			return EDecodeResult::Finished;
		}
	}
	bPreviousIsStreaming = WaveDataRef->IsStreaming();

	int32 NumBytesStreamed = 0;
	while (!bFinished && NumFramesRemaining > 0)
	{
		if (WaveDataRef->IsStreaming())
		{
			NumBytesStreamed = 0;
			bFinished = CompressedAudioInfo->StreamCompressedData(Buff, false, BuffSizeInBytes, NumBytesStreamed);
		}
		else
		{
			NumBytesStreamed = BuffSizeInBytes;
			bFinished = CompressedAudioInfo->ReadCompressedData(Buff, false, BuffSizeInBytes);
		}

		if (CompressedAudioInfo->HasError())
		{
			return EDecodeResult::Fail;
		}

		if (NumBytesStreamed == 0)
		{
			break;
		}

		const int32 NumSamplesStreamed = NumBytesStreamed / sizeof(int16);
		const int32 NumFramesStreamed = NumSamplesStreamed / NumChannels;

		Audio::ArrayPcm16ToFloat(MakeArrayView(ResidualBuffer.GetData(), NumSamplesStreamed), MakeArrayView(SampleConversionBuffer.GetData(), NumSamplesStreamed));

		const float* SampleData = SampleConversionBuffer.GetData();
		DecoderOutput.Push(SampleData, NumSamplesStreamed);

		NumFramesRemaining -= FMath::Min(NumFramesStreamed, NumFramesRemaining);
	}

	if (!bFinished)
	{
		return EDecodeResult::MoreDataRemaining;
	}

	return EDecodeResult::Finished;
}

PRAGMA_ENABLE_DEPRECATION_WARNINGS