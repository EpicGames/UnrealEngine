// Copyright Epic Games, Inc. All Rights Reserved.

#include "SpanAllocator.h"
#include "CoreGlobals.h"

FSpanAllocator::FSpanAllocator(bool bInGrowOnly) 
	: CurrentSize(0)
	, CurrentMaxSize(0)
	, PeakMaxSize(0)
	, FirstNonEmptySpan(0)
	, bGrowOnly(bInGrowOnly)
{}


void FSpanAllocator::Reset()
{
	CurrentSize = 0;
	CurrentMaxSize = 0;
	PeakMaxSize = 0;
	FirstNonEmptySpan = 0;
	FreeSpans.Reset();
	PendingFreeSpans.Reset();
}

void FSpanAllocator::Empty()
{
	CurrentSize = 0;
	CurrentMaxSize = 0;
	PeakMaxSize = 0;
	FirstNonEmptySpan = 0;
	FreeSpans.Empty();
	PendingFreeSpans.Empty();
}

#if DO_CHECK

bool FSpanAllocator::IsFree(int32 Index) const
{
	// If outside the max size, it is considered free as the allocator can grow at will
	if (Index >= CurrentMaxSize)
	{
		return true;
	}

	for (const auto& FreeSpan : FreeSpans)
	{
		if (Index >= FreeSpan.StartOffset && Index < FreeSpan.StartOffset + FreeSpan.Num)
		{
			return true;
		}
	}

	return false;
}

#endif // DO_CHECK

void FSpanAllocator::Consolidate()
{
	// Consolidation 

	if (PendingFreeSpans.IsEmpty() && FirstNonEmptySpan == 0)
	{
		return;
	}

	// 1. Sort the Newly free list by span start, the existing free list is already sorted by construction
	PendingFreeSpans.Sort();

	// alternate free list, used during consolidation to avoid N^2 worst case, retained to avoid re-allocations.
	TArray<FLinearAllocation, TInlineAllocator<10>> FreeSpansTmp;
	FreeSpansTmp.Reset(FreeSpans.Num());

	int32 PrevEndOffset = INDEX_NONE;
	int32 PendingFreeIndex = 0;

	// 2. Joint loop and merge over both free list and newly freed and fuse all adjacent, copy into new free list (to avoid compaction)
	for (int32 Index = 0; Index < FreeSpans.Num() || PendingFreeIndex < PendingFreeSpans.Num(); )
	{
		FLinearAllocation Alloc = Index < FreeSpans.Num() ? FreeSpans[Index] : FLinearAllocation{ MAX_int32, 0 };
		// Make sure we don't run out of both somehow...
		check(PendingFreeIndex < PendingFreeSpans.Num() || Alloc.StartOffset < MAX_int32);
		// Consume the next new alloc if it is before the next old one
		if (PendingFreeIndex < PendingFreeSpans.Num() && PendingFreeSpans[PendingFreeIndex].StartOffset < Alloc.StartOffset)
		{
			Alloc = PendingFreeSpans[PendingFreeIndex++];
		}
		else
		{
			// Otherwise advance the old allocs
			++Index;
		}
		check(Alloc.StartOffset < MAX_int32);

		// Discard empty allocs
		if (Alloc.Num > 0)
		{
			// Continues the previous one, fuse
			if (PrevEndOffset == Alloc.StartOffset)
			{
				FreeSpansTmp.Last().Num += Alloc.Num;
			}
			else
			{
				FreeSpansTmp.Add(Alloc);
			}
			PrevEndOffset = FreeSpansTmp.Last().Num + FreeSpansTmp.Last().StartOffset;
		}
	}

	// Trim last span
	if (!FreeSpansTmp.IsEmpty() && FreeSpansTmp.Last().StartOffset + FreeSpansTmp.Last().Num == CurrentMaxSize)
	{
		CurrentMaxSize -= FreeSpansTmp.Last().Num;
		FreeSpansTmp.Pop(EAllowShrinking::No);
	}

	// 3. Store new free list
	FreeSpans = MoveTemp(FreeSpansTmp);
	PendingFreeSpans.Empty(PendingFreeSpans.Num());
	FirstNonEmptySpan = 0;
}

int32 FSpanAllocator::SearchFreeList(int32 Num, int32 SearchStartIndex)
{
	// Search free list for first matching
	for (int32 Index = SearchStartIndex; Index < FreeSpans.Num(); Index++)
	{
		FLinearAllocation CurrentSpan = FreeSpans[Index];

		if (CurrentSpan.Num >= Num)
		{
			return Index;
		}
	}

	return INDEX_NONE;
}
