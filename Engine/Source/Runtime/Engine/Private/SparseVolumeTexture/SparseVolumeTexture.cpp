// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	SparseVolumeTexture.cpp: SparseVolumeTexture implementation.
=============================================================================*/

#include "SparseVolumeTexture/SparseVolumeTexture.h"

#include "Materials/Material.h"
#include "MaterialShared.h"
#include "UObject/UObjectIterator.h"
#include "Misc/SecureHash.h"
#include "EngineUtils.h"
#include "Shader/ShaderTypes.h"
#include "RenderingThread.h"
#include "GlobalRenderResources.h"
#include "UObject/AssetRegistryTagsContext.h"
#include "UObject/Package.h"
#include "SparseVolumeTexture/SparseVolumeTextureData.h"
#include "SparseVolumeTexture/SparseVolumeTextureUtility.h"
#include "SparseVolumeTexture/ISparseVolumeTextureStreamingManager.h"
#include "ContentStreaming.h"

#if WITH_EDITORONLY_DATA
#include "Misc/ScopedSlowTask.h"
#include "DerivedDataCacheInterface.h"
#include "DerivedDataCache.h"
#include "DerivedDataRequestOwner.h"
#endif
#include "EditorFramework/AssetImportData.h"
#include "Async/ParallelFor.h"

#include "Serialization/LargeMemoryReader.h"
#include "Serialization/LargeMemoryWriter.h"
#include "Serialization/MemoryReader.h"
#include "Serialization/MemoryWriter.h"
#include "Serialization/BulkDataReader.h"
#include "Serialization/BulkDataWriter.h"
#include "Serialization/EditorBulkDataReader.h"
#include "Serialization/EditorBulkDataWriter.h"

#if WITH_EDITOR
#include "Editor.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(SparseVolumeTexture)

#define LOCTEXT_NAMESPACE "USparseVolumeTexture"

DEFINE_LOG_CATEGORY_STATIC(LogSparseVolumeTexture, Log, All);

static int32 GSVTRemoteDDCBehavior = 0;
static FAutoConsoleVariableRef CVarSVTRemoteDDCBehavior(
	TEXT("r.SparseVolumeTexture.RemoteDDCBehavior"),
	GSVTRemoteDDCBehavior,
	TEXT("Controls how SVTs use remote DDC. 0: The bLocalDDCOnly property controls per-SVT caching behavior, 1: Force local DDC only usage for all SVTs, 2: Force local + remote DDC usage for all SVTs"),
	ECVF_Default
);

static float GSVTStreamingRequestMipBias = 1.0f;
static FAutoConsoleVariableRef CVarSVTStreamingRequestMipBias(
	TEXT("r.SparseVolumeTexture.Streaming.RequestMipBias"),
	GSVTStreamingRequestMipBias,
	TEXT("Bias to apply the calculated mip level to stream at. This is used to account for the mip estimation based on projected screen space size being very conservative. ")
	TEXT("The default value of 1.0 was found empirically to roughly result in a 1:1 voxel to pixel ratio."),
	ECVF_Default
);

static int32 GSVTStreamingDDCChunkSize = 2;
static FAutoConsoleVariableRef CVarSVTStreamingDDCChunkSize(
	TEXT("r.SparseVolumeTexture.Streaming.DDCChunkSize"),
	GSVTStreamingDDCChunkSize,
	TEXT("Size of DDC chunks the streaming data is split into (in MiB). A smaller size leads to more requests but can improve streaming performance. Default: 2 MiB"),
	ECVF_Default | ECVF_ReadOnly
);

////////////////////////////////////////////////////////////////////////////////////////////////

FArchive& operator<<(FArchive& Ar, UE::SVT::FMipLevelStreamingInfo& MipLevelStreamingInfo)
{
	Ar << MipLevelStreamingInfo.BulkOffset;
	Ar << MipLevelStreamingInfo.BulkSize;
	Ar << MipLevelStreamingInfo.PageTableOffset;
	Ar << MipLevelStreamingInfo.PageTableSize;
	Ar << MipLevelStreamingInfo.OccupancyBitsOffset;
	Ar << MipLevelStreamingInfo.OccupancyBitsSize;
	Ar << MipLevelStreamingInfo.TileDataOffsetsOffset;
	Ar << MipLevelStreamingInfo.TileDataOffsetsSize;
	Ar << MipLevelStreamingInfo.TileDataOffset;
	Ar << MipLevelStreamingInfo.TileDataSize;
	Ar << MipLevelStreamingInfo.NumPhysicalTiles;
	return Ar;
}

FArchive& operator<<(FArchive& Ar, UE::SVT::FHeader& Header)
{
	Ar << Header.VirtualVolumeResolution;
	Ar << Header.VirtualVolumeAABBMin;
	Ar << Header.VirtualVolumeAABBMax;
	Ar << Header.PageTableVolumeResolution;
	Ar << Header.PageTableVolumeAABBMin;
	Ar << Header.PageTableVolumeAABBMax;
	UE::SVT::Private::SerializeEnumAs<uint8>(Ar, Header.AttributesFormats[0]);
	UE::SVT::Private::SerializeEnumAs<uint8>(Ar, Header.AttributesFormats[1]);
	Ar << Header.FallbackValues[0];
	Ar << Header.FallbackValues[1];
	return Ar;
}

FArchive& operator<<(FArchive& Ar, UE::SVT::FPageTopology::FMip& Mip)
{
	Ar << Mip.PageOffset;
	Ar << Mip.PageCount;
	return Ar;
}

////////////////////////////////////////////////////////////////////////////////////////////////

namespace UE
{
namespace SVT
{

static int32 ComputeNumMipLevels(const FIntVector3& InVirtualVolumeMin, const FIntVector3& InVirtualVolumeMax)
{
	FHeader DummyHeader(InVirtualVolumeMin, InVirtualVolumeMax, PF_Unknown, PF_Unknown, FVector4f(), FVector4f());
	int32 Levels = 1;
	FIntVector3 PageTableResolution = DummyHeader.PageTableVolumeResolution;
	while (PageTableResolution.X > 1 || PageTableResolution.Y > 1 || PageTableResolution.Z > 1)
	{
		PageTableResolution /= 2;
		++Levels;
	}
	return Levels;
};

static const FString& GetDerivedDataVersion()
{
	static FString CachedVersionString = "C9CD1136-8E72-4E0B-A846-15B339414160";	// Bump this if you want to ignore all cached data so far.
	return CachedVersionString;
}

FHeader::FHeader(const FIntVector3& AABBMin, const FIntVector3& AABBMax, EPixelFormat FormatA, EPixelFormat FormatB, const FVector4f& FallbackValueA, const FVector4f& FallbackValueB)
{
	VirtualVolumeAABBMin = AABBMin;
	VirtualVolumeAABBMax = AABBMax;
	VirtualVolumeResolution = VirtualVolumeAABBMax - VirtualVolumeAABBMin;

	PageTableVolumeAABBMin = VirtualVolumeAABBMin / SPARSE_VOLUME_TILE_RES;
	PageTableVolumeAABBMax = (VirtualVolumeAABBMax + FIntVector3(SPARSE_VOLUME_TILE_RES - 1)) / SPARSE_VOLUME_TILE_RES;
	PageTableVolumeResolution = PageTableVolumeAABBMax - PageTableVolumeAABBMin;

	// We need to ensure a power of two resolution for the page table in order to fit all mips of the page table into the physical mips of the texture resource.
	PageTableVolumeResolution.X = FMath::RoundUpToPowerOfTwo(PageTableVolumeResolution.X);
	PageTableVolumeResolution.Y = FMath::RoundUpToPowerOfTwo(PageTableVolumeResolution.Y);
	PageTableVolumeResolution.Z = FMath::RoundUpToPowerOfTwo(PageTableVolumeResolution.Z);
	PageTableVolumeAABBMax = PageTableVolumeAABBMin + PageTableVolumeResolution;

	AttributesFormats[0] = FormatA;
	AttributesFormats[1] = FormatB;

	FallbackValues[0] = FallbackValueA;
	FallbackValues[1] = FallbackValueB;
}

void FHeader::UpdatePageTableFromGlobalNumMipLevels(int32 NumMipLevelsGlobal)
{
	check(NumMipLevelsGlobal < 32);
	const int32 Alignment = 1 << (NumMipLevelsGlobal - 1);

	PageTableVolumeAABBMin = ((VirtualVolumeAABBMin / SPARSE_VOLUME_TILE_RES) / Alignment) * Alignment;
	PageTableVolumeAABBMax = (VirtualVolumeAABBMax + FIntVector3(SPARSE_VOLUME_TILE_RES - 1)) / SPARSE_VOLUME_TILE_RES;
	PageTableVolumeResolution = PageTableVolumeAABBMax - PageTableVolumeAABBMin;

	// We need to ensure a power of two resolution for the page table in order to fit all mips of the page table into the physical mips of the texture resource.
	PageTableVolumeResolution.X = FMath::RoundUpToPowerOfTwo(PageTableVolumeResolution.X);
	PageTableVolumeResolution.Y = FMath::RoundUpToPowerOfTwo(PageTableVolumeResolution.Y);
	PageTableVolumeResolution.Z = FMath::RoundUpToPowerOfTwo(PageTableVolumeResolution.Z);
	PageTableVolumeAABBMax = PageTableVolumeAABBMin + PageTableVolumeResolution;
}

bool FHeader::Validate(bool bPrintToLog)
{
	for (int32 i = 0; i < 2; ++i)
	{
		if (!SVT::IsSupportedFormat(AttributesFormats[i]))
		{
			if (bPrintToLog)
			{
				UE_LOGF(LogSparseVolumeTexture, Warning, "'%ls' is not a supported SparseVolumeTexture format!", GPixelFormats[AttributesFormats[i]].Name);
			}
			return false;
		}
	}

	if (PageTableVolumeResolution.X > SVT::MaxVolumeTextureDim || PageTableVolumeResolution.Y > SVT::MaxVolumeTextureDim || PageTableVolumeResolution.Z > SVT::MaxVolumeTextureDim)
	{
		if (bPrintToLog)
		{
			UE_LOGF(LogSparseVolumeTexture, Warning, "SparseVolumeTexture page table texture dimensions exceed limit (%ix%ix%i): %ix%ix%i",
				SVT::MaxVolumeTextureDim, SVT::MaxVolumeTextureDim, SVT::MaxVolumeTextureDim,
				PageTableVolumeResolution.X, PageTableVolumeResolution.Y, PageTableVolumeResolution.Z);
		}
		return false;
	}

	const int64 PageTableSizeBytes = (int64)PageTableVolumeResolution.X * (int64)PageTableVolumeResolution.Y * (int64)PageTableVolumeResolution.Z * sizeof(uint32);
	if (PageTableSizeBytes > SVT::MaxResourceSize)
	{
		if (bPrintToLog)
		{
			UE_LOGF(LogSparseVolumeTexture, Warning, "SparseVolumeTexture page table texture memory size (%lld) exceeds the 2048MB GPU resource limit!", (long long)PageTableSizeBytes);
		}
		return false;
	}

	return true;
}

////////////////////////////////////////////////////////////////////////////////////////////////

void FPageTopology::Reset()
{
	MipInfo.Reset();
	PackedPageTableCoords.Reset();
	TileIndices.Reset();
	ParentIndices.Reset();
}

void FPageTopology::Serialize(FArchive& Ar)
{
	Ar << MipInfo;
	Ar << PackedPageTableCoords;
	Ar << TileIndices;
	Ar << ParentIndices;
}

void FPageTopology::GetTileRange(uint32 PageOffset, uint32 PageCount, uint32& OutTileOffset, uint32& OutTileCount) const
{
	uint32 TileRangeMin = UINT32_MAX;
	uint32 TileRangeMax = 0;
	for (uint32 PageIndex = PageOffset; PageIndex < (PageOffset + PageCount); ++PageIndex)
	{
		check(IsValidPageIndex(PageIndex));
		const uint32 TileIndex = TileIndices[PageIndex];
		TileRangeMin = FMath::Min(TileIndex, TileRangeMin);
		TileRangeMax = FMath::Max(TileIndex, TileRangeMax);
	}
	if (TileRangeMax >= TileRangeMin)
	{
		OutTileOffset = TileRangeMin;
		OutTileCount = TileRangeMax - TileRangeMin + 1;
	}
	else
	{
		OutTileOffset = 0;
		OutTileCount = 0;
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////

void FResources::Serialize(FArchive& Ar, UObject* Owner, bool bCooked)
{
	// Note: this is all derived data, native versioning is not needed, but be sure to bump GetDerivedDataVersion() when modifying!
	FStripDataFlags StripFlags(Ar, 0);
	if (!StripFlags.IsAudioVisualDataStripped())
	{
		Ar << Header;

		uint32 StoredResourceFlags;
		if (Ar.IsSaving() && bCooked)
		{
			// Disable DDC store when saving out a cooked build
			StoredResourceFlags = ResourceFlags & ~EResourceFlag_StreamingDataInDDC;
			Ar << StoredResourceFlags;
		}
		else
		{
			Ar << ResourceFlags;
			StoredResourceFlags = ResourceFlags;
		}

		Ar << StreamingMetaData.TileDataOffsets;
		Ar << StreamingMetaData.NumVoxelsA;
		Ar << StreamingMetaData.FirstStreamingTileIndex;
		Ar << RootData;

		Topology.Serialize(Ar);

#if WITH_EDITORONLY_DATA
		// These members are only needed when streaming from DDC
		if (!bCooked)
		{
			Ar << DDCChunkIds;
			Ar << DDCChunkMaxTileIndices;
		}
#endif

		// StreamableMipLevels is only serialized in cooked builds and when caching to DDC failed in editor builds.
		// If the data was successfully cached to DDC, we just query it from DDC on the next run or recreate it if that failed.
		if (StoredResourceFlags & EResourceFlag_StreamingDataInDDC)
		{
#if !WITH_EDITORONLY_DATA
			checkf(false, TEXT("UE::SVT::FResources was serialized with EResourceFlag_StreamingDataInDDC in a cooked build!"));
#endif
		}
		else
		{
			StreamableMipLevels.Serialize(Ar, Owner, 0);
		}

#if !WITH_EDITORONLY_DATA
		check(!HasStreamingData() || StreamableMipLevels.GetBulkDataSize() > 0);
#endif
	}
}

bool FResources::HasStreamingData() const
{
	return StreamingMetaData.GetNumStreamingTiles() > 0;
}

#if WITH_EDITORONLY_DATA

void FResources::DropBulkData()
{
	if (HasStreamingData() && (ResourceFlags & EResourceFlag_StreamingDataInDDC))
	{
		StreamableMipLevels.RemoveBulkData();
	}
}

bool FResources::HasBuildFromDDCError() const
{
	return DDCRebuildState.load() == EDDCRebuildState::InitialAfterFailed;
}

void FResources::SetHasBuildFromDDCError(bool bHasError)
{
	if (bHasError)
	{
		EDDCRebuildState ExpectedState = EDDCRebuildState::Initial;
		DDCRebuildState.compare_exchange_strong(ExpectedState, EDDCRebuildState::InitialAfterFailed);
	}
	else
	{
		EDDCRebuildState ExpectedState = EDDCRebuildState::InitialAfterFailed;
		DDCRebuildState.compare_exchange_strong(ExpectedState, EDDCRebuildState::Initial);
	}
}

bool FResources::RebuildBulkDataFromCacheAsync(const UObject* Owner, bool& bFailed)
{
	bFailed = false;
	if (!HasStreamingData() || (ResourceFlags & EResourceFlag_StreamingDataInDDC) == 0u)
	{
		return true;
	}
	if (IsInitialState(DDCRebuildState.load()))
	{
		if (StreamableMipLevels.IsBulkDataLoaded())
		{
			return true;
		}
		// Handle Initial state first so we can transition directly to Succeeded/Failed if the data was immediately available from the cache.
		check(!(*DDCRequestOwner).IsValid());
		BeginRebuildBulkDataFromCache(Owner);
	}
	switch (DDCRebuildState.load())
	{
	case EDDCRebuildState::Pending:
		return false;
	case EDDCRebuildState::Succeeded:
		check(StreamableMipLevels.GetBulkDataSize() > 0);
		EndRebuildBulkDataFromCache();
		return true;
	case EDDCRebuildState::Failed:
		bFailed = true;
		EndRebuildBulkDataFromCache();
		return true;
	default:
		check(false);
		return true;
	}
}

bool FResources::Build(USparseVolumeTextureFrame* Owner, UE::Serialization::FEditorBulkData& SourceData)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(SVT::FResources::Build);

	// Check if the virtualized bulk data payload is available
	if (SourceData.HasPayloadData())
	{
		UE::Serialization::FEditorBulkDataReader SourceDataReader(SourceData);
		FTextureData SourceTextureData;
		SourceDataReader << SourceTextureData;

		FTextureDataAddressingInfo AddressingInfo{};
		AddressingInfo.VolumeResolution = Owner->GetVolumeResolution();
		AddressingInfo.AddressX = Owner->GetTextureAddressX();
		AddressingInfo.AddressY = Owner->GetTextureAddressY();
		AddressingInfo.AddressZ = Owner->GetTextureAddressZ();

		const int32 NumMipLevelsGlobal = Owner->GetNumMipLevels();
		const bool bMoveMip0FromSource = true; // we have no need to keep SourceTextureData around
		FDerivedTextureData DerivedTextureData;
		if (!SourceTextureData.BuildDerivedData(AddressingInfo, NumMipLevelsGlobal, bMoveMip0FromSource, DerivedTextureData))
		{
			return false;
		}

		// Now unload the source data
		SourceData.UnloadData();

		Header = DerivedTextureData.Header;
		NumMipLevels = DerivedTextureData.MipPageRanges.Num();
		RootData.Reset();
		Topology.Reset();
		ResourceFlags = 0;
		ResourceName.Reset();
		DDCKeyHash.Reset();
		DDCChunkIds.Reset();
		DDCChunkMaxTileIndices.Reset();
		DDCRebuildState.store(EDDCRebuildState::Initial);

		// Build page topology
		Topology.Reset();
		Topology.MipInfo.Reserve(DerivedTextureData.MipPageRanges.Num());
		for (const FDerivedTextureData::FMipPageRange& MipRange : DerivedTextureData.MipPageRanges)
		{
			FPageTopology::FMip& Mip = Topology.MipInfo.AddDefaulted_GetRef();
			Mip.PageOffset = MipRange.PageOffset;
			Mip.PageCount = MipRange.PageCount;
		}
		Topology.PackedPageTableCoords = MoveTemp(DerivedTextureData.PageTableCoords);
		Topology.TileIndices = MoveTemp(DerivedTextureData.PageTableTileIndices);
		Topology.ParentIndices = MoveTemp(DerivedTextureData.PageTableParentIndices);

		// Compress tile data
		TArray64<uint8> StreamableBulkData;
		StreamingMetaData = CompressTiles(Topology, DerivedTextureData, RootData, StreamableBulkData);

		// Store StreamableMipLevels
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(SVT::FResources::Build::StoreStreamableMipLevels);

			StreamableMipLevels.Lock(LOCK_READ_WRITE);
			uint8* Ptr = (uint8*)StreamableMipLevels.Realloc(StreamableBulkData.Num());
			FMemory::Memcpy(Ptr, StreamableBulkData.GetData(), StreamableBulkData.Num());
			StreamableMipLevels.Unlock();
			StreamableMipLevels.SetBulkDataFlags(BULKDATA_Force_NOT_InlinePayload);
		}

		return true;
	}
	return false;
}

void FResources::Cache(USparseVolumeTextureFrame* Owner, UE::Serialization::FEditorBulkData& SourceData, bool bLocalCachingOnly)
{
	if (Owner->GetPackage()->bIsCookedForEditor)
	{
		// Don't cache for cooked packages
		return;
	}

	TRACE_CPUPROFILER_EVENT_SCOPE(SVT::FResources::Cache);

	using namespace UE::DerivedData;

	static const FValueId SVTDataId = FValueId::FromName("SparseVolumeTextureData");
	const int32 DDCChunkSizeInMiB = FMath::Max(GSVTStreamingDDCChunkSize, 1);
	const FString KeySuffix = SourceData.GetIdentifier().ToString() + FString::Format(TEXT("{0}_{1}_{2}_{3}_{4}"), { Owner->GetNumMipLevels(), Owner->GetTextureAddressX(), Owner->GetTextureAddressY(), Owner->GetTextureAddressZ(), DDCChunkSizeInMiB });
	FString DerivedDataKey = FDerivedDataCacheInterface::BuildCacheKey(TEXT("SPARSEVOLUMETEXTURE"), *GetDerivedDataVersion(), *KeySuffix);

	FCacheKey CacheKey;
	CacheKey.Bucket = FCacheBucket(TEXT("SparseVolumeTexture"));
	CacheKey.Hash = FIoHash::HashBuffer(MakeMemoryView(FTCHARToUTF8(DerivedDataKey)));

	// Set the cache policy (local only vs local+remote)
	ECachePolicy DefaultCachePolicy = ECachePolicy::Default;
	switch (GSVTRemoteDDCBehavior)
	{
	case 1:	DefaultCachePolicy = ECachePolicy::Local; break;
	case 2:	DefaultCachePolicy = ECachePolicy::Default; break;
	default: DefaultCachePolicy = bLocalCachingOnly ? ECachePolicy::Local : ECachePolicy::Default; break;
	}

	// Check if the data already exists in DDC
	FSharedBuffer ResourcesDataBuffer;

	bool bTryLoadingFromDDC = !HasBuildFromDDCError();
	if (bTryLoadingFromDDC)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(SVT::FResources::Cache::CheckDDC);

		FCacheRecordPolicyBuilder PolicyBuilder(DefaultCachePolicy | ECachePolicy::KeepAlive | ECachePolicy::SkipData);
		PolicyBuilder.AddValuePolicy(SVTDataId, DefaultCachePolicy);

		FCacheGetRequest Request;
		Request.Name = Owner->GetPathName();
		Request.Key = CacheKey;
		Request.Policy = PolicyBuilder.Build();

		FRequestOwner RequestOwner(EPriority::Blocking);
		GetCache().Get(MakeArrayView(&Request, 1), RequestOwner,
			[&ResourcesDataBuffer](FCacheGetResponse&& Response)
			{
				if (Response.Status == EStatus::Ok)
				{
					const FCompressedBuffer& CompressedBuffer = Response.Record.GetValue(SVTDataId).GetData();
					ResourcesDataBuffer = CompressedBuffer.Decompress();
				}
			});
		RequestOwner.Wait();
	}

	if (!ResourcesDataBuffer.IsNull())
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(SVT::FResources::Cache::SerializeFromDDC);

		// Found it!
		// We can serialize the data from the DDC buffer and are done.
		FMemoryReaderView Ar(ResourcesDataBuffer.GetView(), /*bIsPersistent=*/ true);
		Serialize(Ar, Owner, /*bCooked=*/ false);

		check(StreamableMipLevels.GetBulkDataSize() == 0);
		if (ResourceFlags & EResourceFlag_StreamingDataInDDC)
		{
			DDCKeyHash = CacheKey.Hash;
		}
	}
	else
	{
		// DDC lookup failed! Build the data again.
		const bool bBuiltSuccessfully = Build(Owner, SourceData);
		check(bBuiltSuccessfully);

		FCacheRecordBuilder RecordBuilder(CacheKey);
		if (HasStreamingData())
		{
			FByteBulkData& BulkData = StreamableMipLevels;
			const uint8* SrcPtr = static_cast<const uint8*>(BulkData.LockReadOnly());
			const int64 SrcSize = BulkData.GetBulkDataSize();
			const int64 TargetChunkSize = DDCChunkSizeInMiB * 1024LL * 1024LL;
			
			check(DDCChunkMaxTileIndices.IsEmpty());
			check(DDCChunkIds.IsEmpty());
			static_assert(sizeof(FValueId::ByteArray) == 12);

			uint32 FirstTileIndexInChunk = StreamingMetaData.FirstStreamingTileIndex;
			for (uint32 TileIndex = StreamingMetaData.FirstStreamingTileIndex; TileIndex < StreamingMetaData.GetNumTiles(); ++TileIndex)
			{
				bool bCreateChunk = !StreamingMetaData.TileDataOffsets.IsValidIndex(TileIndex + 2); // Create a chunk if this is the last tile.
				if (!bCreateChunk)
				{
					const int64 RangeSizeIncludingNextTile = StreamingMetaData.TileDataOffsets[TileIndex + 2] - StreamingMetaData.TileDataOffsets[FirstTileIndexInChunk];
					bCreateChunk = RangeSizeIncludingNextTile > TargetChunkSize; // Create a chunk if the next tile would exceed the target chunk size
				}

				if (bCreateChunk)
				{
					const int64 ChunkOffset = StreamingMetaData.TileDataOffsets[FirstTileIndexInChunk] - StreamingMetaData.GetRootTileSize();
					const int64 ChunkSize = StreamingMetaData.TileDataOffsets[TileIndex + 1] - StreamingMetaData.TileDataOffsets[FirstTileIndexInChunk];
					const uint8* ChunkPtr = SrcPtr + ChunkOffset;
					const FValue Value = FValue::Compress(FSharedBuffer::MakeView(ChunkPtr, ChunkSize));

					const FString ChunkName = FString::Printf(TEXT("SparseVolumeTextureDataChunk%i"), DDCChunkMaxTileIndices.Num());
					const FValueId ChunkId = FValueId::FromName(ChunkName.GetCharArray());
					const uint8* ChunkIdBytes = ChunkId.GetBytes();

					RecordBuilder.AddValue(ChunkId, Value);

					DDCChunkMaxTileIndices.Add(TileIndex);
					FMemory::Memcpy(DDCChunkIds.AddDefaulted_GetRef().GetData(), ChunkIdBytes, 12);
					FirstTileIndexInChunk = TileIndex + 1;
				}
			}

			BulkData.Unlock();
			ResourceFlags |= EResourceFlag_StreamingDataInDDC;
			DDCKeyHash = CacheKey.Hash;
		}

		// Serialize to a buffer and store into DDC.
		FLargeMemoryWriter Ar(0, /*bIsPersistent=*/ true);
		Serialize(Ar, Owner, /*bCooked=*/ false);

		bool bSavedToDDC = false;
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(SVT::FResources::Cache::SaveToDDC);

			FValue Value = FValue::Compress(FSharedBuffer::MakeView(Ar.GetData(), Ar.TotalSize()));
			RecordBuilder.AddValue(SVTDataId, Value);

			FRequestOwner RequestOwner(UE::DerivedData::EPriority::Blocking);
			const FCachePutRequest PutRequest = { FSharedString(Owner->GetPathName()), RecordBuilder.Build(), DefaultCachePolicy | ECachePolicy::KeepAlive };
			GetCache().Put(MakeArrayView(&PutRequest, 1), RequestOwner,
				[&bSavedToDDC](FCachePutResponse&& Response)
				{
					if (Response.Status == EStatus::Ok)
					{
						bSavedToDDC = true;
					}
				});

			RequestOwner.Wait();

			if (bSavedToDDC)
			{
				SetHasBuildFromDDCError(false);
				if (HasStreamingData())
				{
					// Drop streaming data from memory when it has been successfully committed to DDC
					DropBulkData();
				}
			}
		}

		if (HasStreamingData() && !bSavedToDDC)
		{
			// Streaming data was not pushed to DDC. Disable DDC streaming flag.
			check(StreamableMipLevels.GetBulkDataSize() > 0);
			ResourceFlags &= ~EResourceFlag_StreamingDataInDDC;
		}
	}
}

void FResources::SetDefault(EPixelFormat FormatA, EPixelFormat FormatB, const FVector4f& FallbackValueA, const FVector4f& FallbackValueB)
{
	Header = FHeader(FIntVector(0, 0, 0), FIntVector(1, 1, 1), FormatA, FormatB, FallbackValueA, FallbackValueB);
	ResourceFlags = 0;
	NumMipLevels = 1;
	StreamingMetaData.Reset();
	RootData.Reset();
	StreamableMipLevels.RemoveBulkData();
	ResourceName.Reset();
	DDCKeyHash.Reset();
	DDCChunkIds.Reset();
	DDCChunkMaxTileIndices.Reset();
	DDCRebuildState.store(EDDCRebuildState::Initial);
}

#endif // WITH_EDITORONLY_DATA

#if WITH_EDITORONLY_DATA

void FResources::BeginRebuildBulkDataFromCache(const UObject* Owner)
{
	using namespace UE::DerivedData;

	check(IsInitialState(DDCRebuildState.load()));
	if (!HasStreamingData() || (ResourceFlags & EResourceFlag_StreamingDataInDDC) == 0u)
	{
		return;
	}
	
	check(!DDCKeyHash.IsZero());
	*DDCRequestOwner = MakePimpl<FRequestOwner>(EPriority::Normal);
	DDCRebuildState.store(EDDCRebuildState::Pending);
	DDCRebuildNumFinishedRequests.store(0);

	// Lock and realloc bulk data
	StreamableMipLevels.Lock(LOCK_READ_WRITE);
	const int64 StreamableBulkDataSize = StreamingMetaData.TileDataOffsets.Last() - StreamingMetaData.GetRootTileSize();
	uint8* BulkDataPtr = (uint8*)StreamableMipLevels.Realloc(StreamableBulkDataSize);

	// Generate requests
	const int32 NumChunks = DDCChunkIds.Num();
	check(NumChunks > 0);
	TArray<FCacheGetChunkRequest> DDCRequests;
	DDCRequests.Reserve(NumChunks);
	uint32 FirstTileIndexInChunk = StreamingMetaData.FirstStreamingTileIndex;
	for (int32 ChunkIndex = 0; ChunkIndex < NumChunks; ++ChunkIndex)
	{
		FCacheGetChunkRequest& Request = DDCRequests.AddDefaulted_GetRef();
		Request.Name = Owner->GetPathName();
		Request.Key.Bucket = FCacheBucket(TEXT("SparseVolumeTexture"));
		Request.Key.Hash = DDCKeyHash;
		Request.Id = FValueId(FMemoryView(DDCChunkIds[ChunkIndex].GetData(), 12));
		Request.UserData = StreamingMetaData.TileDataOffsets[FirstTileIndexInChunk] - StreamingMetaData.GetRootTileSize(); // Store write offset in UserData
		FirstTileIndexInChunk = DDCChunkMaxTileIndices[ChunkIndex] + 1;
	}

	// Issue requests
	GetCache().GetChunks(DDCRequests, **DDCRequestOwner,
		[this, BulkDataPtr, NumChunks](FCacheGetChunkResponse&& Response)
		{
			if (Response.Status == EStatus::Ok)
			{
				FMemory::Memcpy(BulkDataPtr + Response.UserData, Response.RawData.GetData(), Response.RawData.GetSize());
				
				// The last request to finish sets the Succeeded flag
				const int32 NumFinishedRequests = DDCRebuildNumFinishedRequests.fetch_add(1) + 1;
				if (NumFinishedRequests == NumChunks)
				{
					DDCRebuildState.store(EDDCRebuildState::Succeeded);
				}
			}
			else
			{
				DDCRebuildState.store(EDDCRebuildState::Failed);
			}
		});
}

void FResources::EndRebuildBulkDataFromCache()
{
	StreamableMipLevels.Unlock();
	StreamableMipLevels.SetBulkDataFlags(BULKDATA_Force_NOT_InlinePayload);
	if (*DDCRequestOwner)
	{
		(*DDCRequestOwner)->Wait();
		(*DDCRequestOwner).Reset();
	}
	EDDCRebuildState NewState = DDCRebuildState.load() != EDDCRebuildState::Failed ?
		EDDCRebuildState::Initial : EDDCRebuildState::InitialAfterFailed;
	DDCRebuildState.store(NewState);
}

#endif // WITH_EDITORONLY_DATA

FTileStreamingMetaData FResources::CompressTiles(const FPageTopology& Topology, const FDerivedTextureData& DerivedTextureData, TArray<uint8>& OutRootBulkData, TArray64<uint8>& OutStreamingBulkData)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(SVT::FResources::Build::CompressTiles);

	const EPixelFormat Formats[] = { DerivedTextureData.Header.AttributesFormats[0], DerivedTextureData.Header.AttributesFormats[1] };
	const int64 FormatSize[] = { GPixelFormats[Formats[0]].BlockBytes, GPixelFormats[Formats[1]].BlockBytes };
	uint8 NullTileValuesU8[2][sizeof(float) * 4] = {};
	int32 NumTextures = 0;
	for (int32 i = 0; i < 2; ++i)
	{
		if (Formats[i] != PF_Unknown)
		{
			++NumTextures;
			SVT::WriteVoxel(0, NullTileValuesU8[i], DerivedTextureData.Header.AttributesFormats[i], DerivedTextureData.Header.FallbackValues[i]);
		}
	}
	const uint8* PhysicalTileData[] = { DerivedTextureData.PhysicalTileDataA.GetData(), DerivedTextureData.PhysicalTileDataB.GetData() };

	const uint32 NumTiles = DerivedTextureData.NumPhysicalTiles;
	const uint32 NumOccupancyWordsPerTexture = NumTiles * SVT::NumOccupancyWordsPerPaddedTile;
	const uint32 NumOccupancyWords = NumTextures * NumOccupancyWordsPerTexture;
	TArray<uint32> OccupancyBits;
	OccupancyBits.SetNumZeroed(NumOccupancyWords);
	uint32* OccupancyBitsPtr[2];
	OccupancyBitsPtr[0] = FormatSize[0] ? OccupancyBits.GetData() : nullptr;
	OccupancyBitsPtr[1] = FormatSize[1] ? (OccupancyBits.GetData() + (FormatSize[0] ? NumOccupancyWordsPerTexture : 0)) : nullptr;

	TStaticArray<TArray<uint16>, 2> NumVoxels;
	NumVoxels[0].SetNumZeroed(NumTiles);
	NumVoxels[1].SetNum((FormatSize[1] > 0) ? NumTiles : 0);

	// Compute occupancy bitmasks and count number of non-fallback voxels per tile
	ParallelFor(NumTiles, [&](int32 TileIndex)
	{
		for (int32 AttributesIdx = 0; AttributesIdx < 2; ++AttributesIdx)
		{
			if (Formats[AttributesIdx] != PF_Unknown)
			{
				NumVoxels[AttributesIdx][TileIndex] = 0;
				for (int64 VoxelIndex = 0; VoxelIndex < SVT::NumVoxelsPerPaddedTile; ++VoxelIndex)
				{
					const uint8* Src = PhysicalTileData[AttributesIdx] + FormatSize[AttributesIdx] * (TileIndex * SVT::NumVoxelsPerPaddedTile + VoxelIndex);
					bool bIsFallbackValue = FMemory::Memcmp(Src, NullTileValuesU8[AttributesIdx], FormatSize[AttributesIdx]) == 0;
					if (!bIsFallbackValue)
					{
						const int64 WordIndex = TileIndex * SVT::NumOccupancyWordsPerPaddedTile + (VoxelIndex / 32);
						OccupancyBitsPtr[AttributesIdx][WordIndex] |= 1u << (static_cast<uint32>(VoxelIndex) % 32u);
						NumVoxels[AttributesIdx][TileIndex]++;
					}
				}
			}
		}
	});

	TArray<uint32> TileDataOffsets;
	TileDataOffsets.SetNum(NumTiles + 1);

	// Compute actual tile data offsets. Each tile stores data like this: | OccupancyBitsA | OccupancyBitsB | VoxelsA | VoxelsB |
	// OccupancyBits A and B always have the same size, but VoxelsA and VoxelsB can vary
	uint32 CurrentOffset = 0;
	for (uint32 TileIndex = 0; TileIndex < NumTiles; ++TileIndex)
	{
		uint32 PrevOffset = CurrentOffset;
		TileDataOffsets[TileIndex] = CurrentOffset;
		CurrentOffset += SVT::NumOccupancyWordsPerPaddedTile * sizeof(uint32) * NumTextures;
		for (int32 AttributesIdx = 0; AttributesIdx < 2; ++AttributesIdx)
		{
			if (Formats[AttributesIdx] != PF_Unknown)
			{
				CurrentOffset += NumVoxels[AttributesIdx][TileIndex] * FormatSize[AttributesIdx];
			}
		}
		checkf(PrevOffset < CurrentOffset, TEXT("SVT streaming data overflowed the uint32 range!"));
	}

	// Write final size at the end of the array so we can compute individual tile sizes as (Offsets[N+1] - Offsets[N])
	TileDataOffsets[NumTiles] = CurrentOffset;

	// Reuse the allocations
	FTileStreamingMetaData StreamingMetaData;
	StreamingMetaData.TileDataOffsets = MoveTemp(TileDataOffsets);
	StreamingMetaData.NumVoxelsA = MoveTemp(NumVoxels[0]);
	StreamingMetaData.FirstStreamingTileIndex = Topology.MipInfo.Last().PageOffset + Topology.MipInfo.Last().PageCount;

	// We're assuming that the very first tile belongs to the root mip and that there is only a maximum of one such tile
	check(Topology.MipInfo.Last().PageOffset == 0);
	check(Topology.MipInfo.Last().PageCount <= 1);
	check(Topology.MipInfo.Last().PageCount == 0 || Topology.TileIndices[0] == 0);

	// Allocate memory for all tiles
	const uint32 RootTileSize = StreamingMetaData.GetRootTileSize();
	OutRootBulkData.SetNum(RootTileSize);
	OutStreamingBulkData.SetNum(CurrentOffset - RootTileSize);

	uint8* RootTileData = OutRootBulkData.GetData();
	uint8* StreamingTileData = OutStreamingBulkData.GetData();

	// Copy tile data to compacted locations
	ParallelFor(NumTiles, [&](uint32 TileIndex)
	{
		const bool bStreamingTile = TileIndex >= StreamingMetaData.FirstStreamingTileIndex;
		uint8* WritePtr = bStreamingTile ? (StreamingTileData + StreamingMetaData.TileDataOffsets[TileIndex] - RootTileSize) : RootTileData;

		const FTileInfo TileInfo = StreamingMetaData.GetTileInfo(TileIndex, FormatSize[0], FormatSize[1]);
		const uint8* BaseWritePtr = WritePtr;

		// Write occupancy bits
		for (int32 AttributesIdx = 0; AttributesIdx < 2; ++AttributesIdx)
		{
			if (Formats[AttributesIdx] != PF_Unknown)
			{
				check(WritePtr == (BaseWritePtr + TileInfo.OccupancyBitsOffsets[AttributesIdx]));
				FMemory::Memcpy(WritePtr, OccupancyBitsPtr[AttributesIdx] + TileIndex * SVT::NumOccupancyWordsPerPaddedTile, SVT::NumOccupancyWordsPerPaddedTile * sizeof(uint32));
				WritePtr += SVT::NumOccupancyWordsPerPaddedTile * sizeof(uint32);
			}
		}

		// Write voxel data
		for (int32 AttributesIdx = 0; AttributesIdx < 2; ++AttributesIdx)
		{
			if (Formats[AttributesIdx] != PF_Unknown)
			{
				check(WritePtr == (BaseWritePtr + TileInfo.VoxelDataOffsets[AttributesIdx]));
				const uint32* TileOccupancyBits = OccupancyBitsPtr[AttributesIdx] + TileIndex * SVT::NumOccupancyWordsPerPaddedTile;

				for (int64 VoxelIndex = 0; VoxelIndex < SVT::NumVoxelsPerPaddedTile; ++VoxelIndex)
				{
					const int64 WordIndex = (VoxelIndex / 32);

					if (TileOccupancyBits[WordIndex] & (1u << (static_cast<uint32>(VoxelIndex) % 32u)))
					{
						const uint8* Src = PhysicalTileData[AttributesIdx] + FormatSize[AttributesIdx] * (TileIndex * SVT::NumVoxelsPerPaddedTile + VoxelIndex);
						FMemory::Memcpy(WritePtr, Src, FormatSize[AttributesIdx]);
						WritePtr += FormatSize[AttributesIdx];
					}
				}
			}
		}
		check((WritePtr - BaseWritePtr) == TileInfo.Size);
	});
	
	return StreamingMetaData;
}

////////////////////////////////////////////////////////////////////////////////////////////////

void FTextureRenderResources::GetPackedUniforms(FUintVector4& OutPacked0, FUintVector4& OutPacked1) const
{
	check(IsInParallelRenderingThread());

	const FIntVector3 PageTableOffset = Header.PageTableVolumeAABBMin;
	const FVector3f TileDataTexelSize = FVector3f(
		1.0f / TileDataTextureResolution.X,
		1.0f / TileDataTextureResolution.Y,
		1.0f / TileDataTextureResolution.Z);
	const FVector3f VolumePageResolution = FVector3f(GlobalVolumeResolution) / SPARSE_VOLUME_TILE_RES;

	OutPacked0.X = FMath::AsUInt(VolumePageResolution.X);
	OutPacked0.Y = FMath::AsUInt(VolumePageResolution.Y);
	OutPacked0.Z = FMath::AsUInt(VolumePageResolution.Z);
	OutPacked0.W = SVT::PackX11Y11Z10(PageTableOffset);
	OutPacked1.X = FMath::AsUInt(TileDataTexelSize.X);
	OutPacked1.Y = FMath::AsUInt(TileDataTexelSize.Y);
	OutPacked1.Z = FMath::AsUInt(TileDataTexelSize.Z);
	OutPacked1.W = 0;
	OutPacked1.W |= (uint32)((FrameIndex & 0xFFFF) << 0);
	OutPacked1.W |= (uint32)(((NumLogicalMipLevels - 1) & 0xFFFF) << 16);
}

void FTextureRenderResources::SetGlobalVolumeResolution_GameThread(const FIntVector3& InGlobalVolumeResolution)
{
	ENQUEUE_RENDER_COMMAND(FTextureRenderResources_UpdateGlobalVolumeResolution)(
		[this, InGlobalVolumeResolution](FRHICommandListImmediate& RHICmdList)
		{
			GlobalVolumeResolution = InGlobalVolumeResolution;
		});
}

void FTextureRenderResources::InitRHI(FRHICommandListBase& RHICmdList)
{
	PageTableTextureReferenceRHI = RHICmdList.CreateTextureReference(GBlackUintVolumeTexture->TextureRHI);
	PhysicalTileDataATextureReferenceRHI = RHICmdList.CreateTextureReference(GBlackVolumeTexture->TextureRHI);
	PhysicalTileDataBTextureReferenceRHI = RHICmdList.CreateTextureReference(GBlackVolumeTexture->TextureRHI);
}

void FTextureRenderResources::ReleaseRHI()
{
	PageTableTextureReferenceRHI.SafeRelease();
	PhysicalTileDataATextureReferenceRHI.SafeRelease();
	PhysicalTileDataBTextureReferenceRHI.SafeRelease();
}

FTileInfo FTileStreamingMetaData::GetTileInfo(uint32 TileIndex, uint32 FormatSizeA, uint32 FormatSizeB) const
{
	const uint32 NumVoxelsInA = NumVoxelsA[TileIndex];
	const uint32 VoxelDataSizeA = NumVoxelsInA * FormatSizeA;

	FTileInfo Result;
	Result.Offset = TileDataOffsets[TileIndex];
	Result.Size = TileDataOffsets[TileIndex + 1] - Result.Offset;
	Result.OccupancyBitsOffsets[0] = 0;
	Result.OccupancyBitsOffsets[1] = FormatSizeA > 0 ? SVT::OccupancyBitsSizePerPaddedTile : 0;
	Result.OccupancyBitsSizes[0] = FormatSizeA > 0 ? SVT::OccupancyBitsSizePerPaddedTile : 0;
	Result.OccupancyBitsSizes[1] = FormatSizeB > 0 ? SVT::OccupancyBitsSizePerPaddedTile : 0;
	Result.VoxelDataOffsets[0] = Result.OccupancyBitsOffsets[1] + (FormatSizeB > 0 ? SVT::OccupancyBitsSizePerPaddedTile : 0);
	Result.VoxelDataOffsets[1] = Result.VoxelDataOffsets[0] + VoxelDataSizeA;
	Result.VoxelDataSizes[0] = VoxelDataSizeA;
	Result.VoxelDataSizes[1] = Result.Size - Result.VoxelDataOffsets[1];
	Result.NumVoxels[0] = NumVoxelsInA;
	Result.NumVoxels[1] = FormatSizeB > 0 ? (Result.VoxelDataSizes[1] / FormatSizeB) : 0;
	
	if (TileIndex >= FirstStreamingTileIndex)
	{
		Result.Offset -= TileDataOffsets[FirstStreamingTileIndex];
	}

	check((Result.OccupancyBitsSizes[0] + Result.OccupancyBitsSizes[1] + Result.VoxelDataSizes[0] + Result.VoxelDataSizes[1]) == Result.Size);
	check((Result.OccupancyBitsOffsets[0] + Result.OccupancyBitsSizes[0]) == Result.OccupancyBitsOffsets[1]);
	check((Result.OccupancyBitsOffsets[1] + Result.OccupancyBitsSizes[1]) == Result.VoxelDataOffsets[0]);
	check((Result.VoxelDataOffsets[0] + Result.VoxelDataSizes[0]) == Result.VoxelDataOffsets[1]);
	check((Result.VoxelDataOffsets[1] + Result.VoxelDataSizes[1]) == Result.Size);

	return Result;
}

void FTileStreamingMetaData::GetNumVoxelsInTileRange(uint32 TileOffset, uint32 TileCount, uint32 FormatSizeA, uint32 FormatSizeB, const TBitArray<>* OptionalValidTiles, uint32& OutNumVoxelsA, uint32& OutNumVoxelsB) const
{
	check(FormatSizeA > 0 || FormatSizeB > 0);
	const uint32 NumTextures = (FormatSizeA > 0 && FormatSizeB > 0) ? 2 : 1;
	const uint32 TileOccupancyBitsSize = NumTextures * SVT::OccupancyBitsSizePerPaddedTile;
	OutNumVoxelsA = 0;
	OutNumVoxelsB = 0;
	for (uint32 TileIndex = TileOffset; TileIndex < (TileOffset + TileCount); ++TileIndex)
	{
		if (OptionalValidTiles && !(*OptionalValidTiles)[TileIndex])
		{
			continue;
		}
		const uint32 NumVoxelsATmp = NumVoxelsA[TileIndex];
		OutNumVoxelsA += NumVoxelsATmp;
		// For NumVoxelsB, we need to reconstruct that value based on the total tile size and the sizes of the other memory sections in the tile
		if (FormatSizeB > 0)
		{
			const uint32 TileSize = GetTileMemorySize(TileIndex);
			const uint32 VoxelsDataOffsetB = TileOccupancyBitsSize + NumVoxelsATmp * FormatSizeA;
			check(TileSize >= VoxelsDataOffsetB);
			OutNumVoxelsB += (TileSize - VoxelsDataOffsetB) / FormatSizeB;
		}
	}
}

}
}

////////////////////////////////////////////////////////////////////////////////////////////////

USparseVolumeTexture::USparseVolumeTexture(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

float USparseVolumeTexture::GetOptimalStreamingMipLevel(const FBoxSphereBounds& Bounds, float MipBias) const
{
	check(IsInGameThread());
	float ResultMipLevel = 0.0f;
	if (IStreamingManager* StreamingManager = IStreamingManager::Get_Concurrent())
	{
		ResultMipLevel = FLT_MAX;
		const int32 NumViews = StreamingManager->GetNumViews();
		for (int32 ViewIndex = 0; ViewIndex < NumViews; ++ViewIndex)
		{
			const FStreamingViewInfo& ViewInfo = StreamingManager->GetViewInformation(ViewIndex);

			// Determine the pixel-width at the near-plane.
			const float PixelWidth = 1.0f / (ViewInfo.FOVScreenSize * 0.5f); // FOVScreenSize = ViewRect.Width / Tan(FOV * 0.5)

			// Project to nearest distance of volume bounds.
			const float Distance = FMath::Max<float>(1.0f, ((ViewInfo.ViewOrigin - Bounds.Origin).GetAbs() - Bounds.BoxExtent).Length());
			const float VoxelWidth = Distance * PixelWidth;

			// MIP is defined as the log of the ratio of native voxel resolution to pixel-coverage of volume bounds.
			// We want to be conservative here (use potentially lower mip), so try to minimize the term we pass into Log2() by using
			// the maximum dimension of the bounds and the minimum extent of the volume resolution. The bounds are axis aligned, so
			// we can't assume that a given dimension in SVT UV space aligns with any particular dimension of the axis aligned bounds.
			const float PixelWidthCoverage = (2.0f * Bounds.BoxExtent.GetMax()) / VoxelWidth;
			const float VoxelResolution = GetVolumeResolution().GetMin();
			float ViewMipLevel = FMath::Log2(VoxelResolution / PixelWidthCoverage) + MipBias + GSVTStreamingRequestMipBias;
			ViewMipLevel = FMath::Clamp(ViewMipLevel, 0.0f, GetNumMipLevels() - 1.0f);

			ResultMipLevel = FMath::Min(ViewMipLevel, ResultMipLevel);
		}
	}
	return ResultMipLevel;
}

UE::Shader::EValueType USparseVolumeTexture::GetUniformParameterType(int32 Index)
{
	switch (Index)
	{
	case ESparseVolumeTexture_TileSize:				return UE::Shader::EValueType::Float1;
	case ESparseVolumeTexture_PageTableSize:		return UE::Shader::EValueType::Float3;
	case ESparseVolumeTexture_UVScale:				return UE::Shader::EValueType::Float3;
	case ESparseVolumeTexture_UVBias:				return UE::Shader::EValueType::Float3;
	default:
		break;
	}
	checkNoEntry();
	return UE::Shader::EValueType::Float4;
}

#if WITH_EDITOR
void USparseVolumeTexture::NotifyMaterials(const ENotifyMaterialsEffectOnShaders EffectOnShaders)
{
	// Create a material update context to safely update materials.
	{
		FMaterialUpdateContext UpdateContext;

		// Notify any material that uses this texture
		TSet<UMaterial*> BaseMaterialsThatUseThisTexture;
		for (TObjectIterator<UMaterialInterface> It; It; ++It)
		{
			UMaterialInterface* MaterialInterface = *It;
			if (!FPlatformProperties::IsServerOnly() && (MaterialInterface->ContainsDefaultTexture(this)))
			{
				UpdateContext.AddMaterialInterface(MaterialInterface);
				// This is a bit tricky. We want to make sure all materials using this texture are
				// updated. Materials are always updated. Material instances may also have to be
				// updated and if they have static permutations their children must be updated
				// whether they use the texture or not! The safe thing to do is to add the instance's
				// base material to the update context causing all materials in the tree to update.
				BaseMaterialsThatUseThisTexture.Add(MaterialInterface->GetMaterial());
			}
		}

		// Go ahead and update any base materials that need to be.
		if (EffectOnShaders == ENotifyMaterialsEffectOnShaders::Default)
		{
			for (TSet<UMaterial*>::TConstIterator It(BaseMaterialsThatUseThisTexture); It; ++It)
			{
				(*It)->PostEditChange();
			}
		}
		else
		{
			FPropertyChangedEvent EmptyPropertyUpdateStruct(nullptr);
			for (TSet<UMaterial*>::TConstIterator It(BaseMaterialsThatUseThisTexture); It; ++It)
			{
				(*It)->PostEditChangePropertyInternal(EmptyPropertyUpdateStruct, UMaterial::EPostEditChangeEffectOnShaders::DoesNotInvalidate);
			}
		}
	}
}
#endif

////////////////////////////////////////////////////////////////////////////////////////////////

USparseVolumeTextureFrame::USparseVolumeTextureFrame(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

USparseVolumeTextureFrame* USparseVolumeTextureFrame::GetFrameAndIssueStreamingRequest(USparseVolumeTexture* SparseVolumeTexture, uint32 StreamingInstanceKey, float FrameRate, float FrameIndex, float MipLevel, bool bBlocking, bool bHasValidFrameRate)
{
	if (UStreamableSparseVolumeTexture* StreamableSVT = Cast<UStreamableSparseVolumeTexture>(SparseVolumeTexture))
	{
		UE::SVT::EStreamingRequestFlags RequestFlags = UE::SVT::EStreamingRequestFlags::None;
		RequestFlags |= bBlocking ? UE::SVT::EStreamingRequestFlags::Blocking : UE::SVT::EStreamingRequestFlags::None;
		RequestFlags |= bHasValidFrameRate ? UE::SVT::EStreamingRequestFlags::HasFrameRate : UE::SVT::EStreamingRequestFlags::None;
		UE::SVT::GetStreamingManager().Request_GameThread(StreamableSVT, StreamingInstanceKey, FrameRate, FrameIndex, MipLevel, RequestFlags);
		return StreamableSVT->GetFrame(static_cast<int32>(FrameIndex));
	}
	return nullptr;
}

bool USparseVolumeTextureFrame::Initialize(USparseVolumeTexture* InOwner, int32 InFrameIndex, const FTransform& InFrameTransform, UE::SVT::FTextureData& UncookedFrame)
{
#if WITH_EDITORONLY_DATA
	Owner = InOwner;
	FrameIndex = InFrameIndex;
	Transform = InFrameTransform;
	{
		UE::Serialization::FEditorBulkDataWriter SourceDataArchiveWriter(SourceData);
		SourceDataArchiveWriter << UncookedFrame;
	}

	return true;
#else
	return false;
#endif
}

bool USparseVolumeTextureFrame::CreateTextureRenderResources()
{
	if (!TextureRenderResources && !IsTemplate() && FApp::CanEverRender())
	{
		TextureRenderResources = new UE::SVT::FTextureRenderResources();
		TextureRenderResources->SetGlobalVolumeResolution_GameThread(Owner->GetVolumeResolution());
		BeginInitResource(TextureRenderResources);

		return true;
	}
	return false;
}

void USparseVolumeTextureFrame::PostLoad()
{
	Super::PostLoad();

	CreateTextureRenderResources();
}

void USparseVolumeTextureFrame::FinishDestroy()
{
	Super::FinishDestroy();
}

void USparseVolumeTextureFrame::BeginDestroy()
{
	// Ensure that the streamable SVT has been removed from the streaming manager
	
	if (!IsTemplate())
	{
		if (IsValid(Owner) && FApp::CanEverRender())
		{
			UStreamableSparseVolumeTexture* SVTOwner = CastChecked<UStreamableSparseVolumeTexture>(Owner);
			for (int i = 0; i < SVTOwner->GetNumFrames(); ++i)
			{
				// if the owner contains the current frame being deleted, remove the owner from the streaming manager
				// SVT_TODO: This is a temporary fix for a GC problem.  In the future this will be replaced with a more robust solution.
				if (SVTOwner->GetFrame(i) == this)
				{
					UE::SVT::GetStreamingManager().Remove_GameThread(SVTOwner);
					break;
				}
			}
		}

		if (TextureRenderResources)
		{
			ENQUEUE_RENDER_COMMAND(USparseVolumeTextureFrame_DeleteTextureRenderResources)(
				[Resources = TextureRenderResources](FRHICommandListImmediate& RHICmdList)
				{
					Resources->ReleaseResource();
					delete Resources;
				});
			TextureRenderResources = nullptr;
		}
	}

	Super::BeginDestroy();
}

void USparseVolumeTextureFrame::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

#if WITH_EDITORONLY_DATA
	if (!Ar.IsCooking())
	{
		SourceData.Serialize(Ar, this);
	}
#endif

	bool bCooked = Ar.IsCooking();
	Ar << bCooked;

	// Inline the derived data for cooked builds.
	if (bCooked && !IsTemplate() && !Ar.IsCountingMemory())
	{
		Resources.Serialize(Ar, this, bCooked);
	}
}

void USparseVolumeTextureFrame::GetResourceSizeEx(FResourceSizeEx& CumulativeResourceSize)
{
	Super::GetResourceSizeEx(CumulativeResourceSize);
}

#if WITH_EDITOR

void USparseVolumeTextureFrame::BeginCacheForCookedPlatformData(const ITargetPlatform* TargetPlatform)
{
}

bool USparseVolumeTextureFrame::IsCachedCookedPlatformDataLoaded(const ITargetPlatform* TargetPlatform)
{
	bool bFailed = false;
	if (!Resources.RebuildBulkDataFromCacheAsync(this, bFailed))
	{
		return false;
	}

	if (bFailed)
	{
		UE_LOGF(LogSparseVolumeTexture, Log, "Failed to recover SparseVolumeTexture streaming from DDC for '%ls' Frame %i. Rebuilding and retrying.", *Owner->GetPathName(), FrameIndex);

		Resources.Cache(this, SourceData, CastChecked<UStreamableSparseVolumeTexture>(Owner)->bLocalDDCOnly);
		return false;
	}

	return true;
}

void USparseVolumeTextureFrame::WillNeverCacheCookedPlatformDataAgain()
{
}

void USparseVolumeTextureFrame::ClearCachedCookedPlatformData(const ITargetPlatform* TargetPlatform)
{
	Resources.DropBulkData();
}

void USparseVolumeTextureFrame::ClearAllCachedCookedPlatformData()
{
	Resources.DropBulkData();
}

#endif // WITH_EDITOR

#if WITH_EDITORONLY_DATA
void USparseVolumeTextureFrame::Cache(bool bSkipDDCAndSetResourcesToDefault)
{
	if (bSkipDDCAndSetResourcesToDefault)
	{
		Resources.SetDefault(GetFormat(0), GetFormat(1), GetFallbackValue(0), GetFallbackValue(1));
	}
	else
	{
		Resources.Cache(this, SourceData, CastChecked<UStreamableSparseVolumeTexture>(Owner)->bLocalDDCOnly);
	}
	CreateTextureRenderResources();
}
#endif

////////////////////////////////////////////////////////////////////////////////////////////////

UStreamableSparseVolumeTexture::UStreamableSparseVolumeTexture(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
#if WITH_EDITOR
	if (!IsTemplate())
	{
		if (HasAnyFlags(RF_NeedPostLoad) || GetOuter()->HasAnyFlags(RF_NeedPostLoad))
		{
			// Delegate registration is not thread-safe, so we postpone it on PostLoad when coming from loading which could be on another thread
		}
		else
		{
			RegisterEditorDelegates();
		}
	}
#endif // WITH_EDITOR
}

bool UStreamableSparseVolumeTexture::BeginInitialize(int32 NumExpectedFrames)
{
#if WITH_EDITORONLY_DATA
	if (InitState != EInitState_Uninitialized)
	{
		UE_LOGF(LogSparseVolumeTexture, Error, "Tried to call UStreamableSparseVolumeTexture::BeginInitialize() while not in the Uninitialized init state.");
		return false;
	}

	Modify();

	check(Frames.IsEmpty());
	Frames.Empty(NumExpectedFrames);
	VolumeBoundsMin = FIntVector(INT32_MAX, INT32_MAX, INT32_MAX);
	VolumeBoundsMax = FIntVector(INT32_MIN, INT32_MIN, INT32_MIN);
	check(FormatA == PF_Unknown);
	check(FormatB == PF_Unknown);

	// This is different to other texture types which all seem to default to wrap. However, for SVT content it is more common to want a single volume without wrapping
	// and since changing the addressing mode currently results in fairly costly recomputing of derived data, defaulting to clamp should result in a better user experience in most cases.
	AddressX = TA_Clamp;
	AddressY = TA_Clamp;
	AddressZ = TA_Clamp;

	InitState = EInitState_Pending;

	return true;
#else
	return false;
#endif
}

bool UStreamableSparseVolumeTexture::AppendFrame(UE::SVT::FTextureData& UncookedFrame, const FTransform& FrameTransform)
{
#if WITH_EDITORONLY_DATA
	if (InitState != EInitState_Pending)
	{
		UE_LOGF(LogSparseVolumeTexture, Error, "Tried to call UStreamableSparseVolumeTexture::AppendFrame() while not in the Pending init state.");
		return false;
	}

	// The the minimum of the union of all frame AABBs should ideally be at (0, 0, 0), but it should also be fine if it is greater than that.
	// A mimimum of less than (0, 0, 0) is not permitted.
	if (UncookedFrame.Header.VirtualVolumeAABBMin.X < 0 || UncookedFrame.Header.VirtualVolumeAABBMin.Y < 0 || UncookedFrame.Header.VirtualVolumeAABBMin.Z < 0)
	{
		UE_LOGF(LogSparseVolumeTexture, Error, "Tried to add a frame to a SparseVolumeTexture with a VirtualVolumeAABBMin < 0 (%i, %i, %i)",
			VolumeBoundsMin.X, VolumeBoundsMin.Y, VolumeBoundsMin.Z);
		return false;
	}

	if (Frames.IsEmpty())
	{
		if (!UE::SVT::IsSupportedFormat(UncookedFrame.Header.AttributesFormats[0]) || !UE::SVT::IsSupportedFormat(UncookedFrame.Header.AttributesFormats[1]))
		{
			UE_LOGF(LogSparseVolumeTexture, Error, "Tried to add a frame with unsupported formats to a SparseVolumeTexture! Formats: (%i %i)",
				(int)UncookedFrame.Header.AttributesFormats[0], (int)UncookedFrame.Header.AttributesFormats[1]);
			return false;
		}
		FormatA = UncookedFrame.Header.AttributesFormats[0];
		FormatB = UncookedFrame.Header.AttributesFormats[1];
		FallbackValueA = UncookedFrame.Header.FallbackValues[0];
		FallbackValueB = UncookedFrame.Header.FallbackValues[1];
	}
	else
	{
		if (UncookedFrame.Header.AttributesFormats[0] != FormatA || UncookedFrame.Header.AttributesFormats[1] != FormatB)
		{
			UE_LOGF(LogSparseVolumeTexture, Error, "Tried to add a frame to a SparseVolumeTexture without matching formats! Expected: (%i, %i), Actual: (%i %i)",
				(int)FormatA.GetIntValue(), (int)FormatB.GetIntValue(), (int)UncookedFrame.Header.AttributesFormats[0], (int)UncookedFrame.Header.AttributesFormats[1]);
			return false;
		}
		if (UncookedFrame.Header.FallbackValues[0] != FallbackValueA || UncookedFrame.Header.FallbackValues[1] != FallbackValueB)
		{
			UE_LOGF(LogSparseVolumeTexture, Error, "Tried to add a frame to a SparseVolumeTexture without matching fallback/null tile values!");
			return false;
		}
	}

	// Compute union of all frame AABBs
	VolumeBoundsMin.X = FMath::Min(VolumeBoundsMin.X, UncookedFrame.Header.VirtualVolumeAABBMin.X);
	VolumeBoundsMin.Y = FMath::Min(VolumeBoundsMin.Y, UncookedFrame.Header.VirtualVolumeAABBMin.Y);
	VolumeBoundsMin.Z = FMath::Min(VolumeBoundsMin.Z, UncookedFrame.Header.VirtualVolumeAABBMin.Z);
	VolumeBoundsMax.X = FMath::Max(VolumeBoundsMax.X, UncookedFrame.Header.VirtualVolumeAABBMax.X);
	VolumeBoundsMax.Y = FMath::Max(VolumeBoundsMax.Y, UncookedFrame.Header.VirtualVolumeAABBMax.Y);
	VolumeBoundsMax.Z = FMath::Max(VolumeBoundsMax.Z, UncookedFrame.Header.VirtualVolumeAABBMax.Z);

	if (VolumeBoundsMin.X >= VolumeBoundsMax.X || VolumeBoundsMin.Y >= VolumeBoundsMax.Y || VolumeBoundsMin.Z >= VolumeBoundsMax.Z)
	{
		// Force a minimum resolution of (1, 1, 1) if the frames were empty so far
		VolumeResolution = FIntVector3(1, 1, 1);
	}
	else
	{
		VolumeResolution = VolumeBoundsMax;
	}
	
	const int32 FrameIndex = Frames.Num();
	const FName FrameBaseName = FName(FString::Printf(TEXT("%s_Frame%i"), *GetFName().ToString(), FrameIndex));
	const FName FrameName = MakeUniqueObjectName(this, USparseVolumeTextureFrame::StaticClass(), FrameBaseName);
	USparseVolumeTextureFrame* Frame = NewObject<USparseVolumeTextureFrame>(this, USparseVolumeTextureFrame::StaticClass(), FrameName, RF_Public);
	if (Frame->Initialize(this, FrameIndex, FrameTransform, UncookedFrame))
	{
		Frames.Add(Frame);
		return true;
	}
	return false;
	
#else
	return false;
#endif
}

bool UStreamableSparseVolumeTexture::EndInitialize()
{
#if WITH_EDITORONLY_DATA
	if (InitState != EInitState_Pending)
	{
		UE_LOGF(LogSparseVolumeTexture, Error, "Tried to call UStreamableSparseVolumeTexture::EndInitialize() while not in the Pending init state.");
		return false;
	}

	// Ensure that at least one frame of data exists
	if (Frames.IsEmpty())
	{
		UE_LOGF(LogSparseVolumeTexture, Warning, "SVT has zero frames! Adding a dummy frame. SVT: %ls", *GetName());
		UE::SVT::FTextureData DummyFrame;
		DummyFrame.CreateDefault();
		AppendFrame(DummyFrame, FTransform::Identity);
	}

	check(VolumeResolution.X > 0 && VolumeResolution.Y > 0 && VolumeResolution.Z > 0);
	check(VolumeBoundsMin.X >= 0 && VolumeBoundsMin.Y >= 0 && VolumeBoundsMin.Z >= 0);
	check(FormatA != PF_Unknown || FormatB != PF_Unknown);

	if (VolumeBoundsMin.X > 0 || VolumeBoundsMin.Y > 0 || VolumeBoundsMin.Z > 0)
	{
		UE_LOGF(LogSparseVolumeTexture, Warning, "Initialized a SparseVolumeTexture with a VirtualVolumeAABBMin > 0 (%i, %i, %i). This wastes memory",
			VolumeBoundsMin.X, VolumeBoundsMin.Y, VolumeBoundsMin.Z);
	}

	const int32 NumMipLevelsFullMipChain = UE::SVT::ComputeNumMipLevels(VolumeBoundsMin, VolumeBoundsMax);
	check(NumMipLevelsFullMipChain > 0);

	NumMipLevels = NumMipLevelsFullMipChain;
	NumFrames = Frames.Num();

	for (USparseVolumeTextureFrame* Frame : Frames)
	{
		Frame->PostLoad();
	}

	InitState = EInitState_Done;

	return true;
#else
	return false;
#endif
}

bool UStreamableSparseVolumeTexture::Initialize(const TArrayView<UE::SVT::FTextureData>& InUncookedData, const TArrayView<FTransform>& InFrameTransforms)
{
	if (InUncookedData.IsEmpty())
	{
		UE_LOGF(LogSparseVolumeTexture, Error, "Tried to initialize a SparseVolumeTexture with no frames");
		return false;
	}

	const int32 NumUncookedFrameData = InUncookedData.Num();
	const int32 NumFrameTransforms = InFrameTransforms.Num();
	const bool bHasValidFrameTransforms = NumUncookedFrameData <= NumFrameTransforms;
	if (!BeginInitialize(NumUncookedFrameData))
	{
		return false;
	}
	for (int32 i = 0; i < NumUncookedFrameData; ++i)
	{
		if (!AppendFrame(InUncookedData[i], bHasValidFrameTransforms ? InFrameTransforms[i] : FTransform::Identity))
		{
			return false;
		}
	}
	if (!EndInitialize())
	{
		return false;
	}

	return true;
}

void UStreamableSparseVolumeTexture::PostInitProperties()
{
	Super::PostInitProperties();

#if WITH_EDITORONLY_DATA
	if (!HasAnyFlags(RF_ClassDefaultObject))
	{
		AssetImportData = NewObject<UAssetImportData>(this, TEXT("AssetImportData"));
	}
#endif
}

void UStreamableSparseVolumeTexture::PostLoad()
{
	Super::PostLoad();

#if WITH_EDITOR
	if (!IsTemplate())
	{
		RegisterEditorDelegates();
	}
#endif

	// Ensure that NumFrames always corresponds to the actual number of frames
	NumFrames = GetNumFrames();

	if (!IsTemplate())
	{
#if WITH_EDITORONLY_DATA
		RecacheFrames();
#else
		if (FApp::CanEverRender())
		{
			for (USparseVolumeTextureFrame* Frame : Frames)
			{
				check(Frame); // Elements in Frames should only ever be null when the SVT is being deleted
				Frame->CreateTextureRenderResources();
			}
			UE::SVT::GetStreamingManager().Add_GameThread(this); // RecacheFrames() handles this in editor builds
		}
#endif
	}
}

void UStreamableSparseVolumeTexture::FinishDestroy()
{
	Super::FinishDestroy();
}

void UStreamableSparseVolumeTexture::BeginDestroy()
{
	Super::BeginDestroy();
	
	if (!IsTemplate())
	{
		if (FApp::CanEverRender())
		{
			UE::SVT::GetStreamingManager().Remove_GameThread(this);
		}

#if WITH_EDITOR
		UnregisterEditorDelegates();
#endif
	}
}

void UStreamableSparseVolumeTexture::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

#if WITH_EDITORONLY_DATA
	// Check that we are not trying to cook unitialized data!
	check(!Ar.IsCooking() || InitState == EInitState_Done);
#endif
}

#if WITH_EDITOR
void UStreamableSparseVolumeTexture::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	// It's possible for outside code/GC to null the elements in Frames. This happens when the SVT is deleted and all the child objects are also first deleted and their references nulled.
	bool bInvalidFramesArray = false;
	for (USparseVolumeTextureFrame* Frame : Frames)
	{
		if (!Frame)
		{
			bInvalidFramesArray = true;
			break;
		}
	}

	if (PropertyChangedEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(UStreamableSparseVolumeTexture, AddressX)
		|| PropertyChangedEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(UStreamableSparseVolumeTexture, AddressY)
		|| PropertyChangedEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(UStreamableSparseVolumeTexture, AddressZ))
	{
		// SVT need to recompile shaders when address mode changes
		NotifyMaterials();
		for (USparseVolumeTextureFrame* Frame : Frames)
		{
			if (Frame)
			{
				Frame->NotifyMaterials();
			}
		}
	}

	Super::PostEditChangeProperty(PropertyChangedEvent);

	// Don't bother trying to recache the frame data if the Frames array is invalid. This very likely means that this object is about to be deleted.
	bool bRecacheFrames = !bInvalidFramesArray;

	if (PropertyChangedEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(UStreamableSparseVolumeTexture, StreamingPoolSizeFactor)
		|| PropertyChangedEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(UStreamableSparseVolumeTexture, NumberOfPrefetchFrames)
		|| PropertyChangedEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(UStreamableSparseVolumeTexture, PrefetchPercentageStepSize)
		|| PropertyChangedEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(UStreamableSparseVolumeTexture, PrefetchPercentageBias))
	{
		// Re-register the SVT with the streamer so it picks up on the changed streaming parameters.
		if (FApp::CanEverRender() && !IsTemplate())
		{
			UE::SVT::GetStreamingManager().Remove_GameThread(this);
			UE::SVT::GetStreamingManager().Add_GameThread(this);
		}
		
		bRecacheFrames = false;
	}
	
	if (bRecacheFrames)
	{
		RecacheFrames();
	}
}
#endif // WITH_EDITOR

void UStreamableSparseVolumeTexture::GetResourceSizeEx(FResourceSizeEx& CumulativeResourceSize)
{
	Super::GetResourceSizeEx(CumulativeResourceSize);
	SIZE_T SizeCPU = sizeof(*this) - sizeof(Super);
	SIZE_T SizeGPU = 0;
	SizeCPU += Frames.GetAllocatedSize();
	for (USparseVolumeTextureFrame* Frame : Frames)
	{
		if (Frame)
		{
			Frame->GetResourceSizeEx(CumulativeResourceSize);
		}
	}
	CumulativeResourceSize.AddDedicatedSystemMemoryBytes(SizeCPU);
	CumulativeResourceSize.AddDedicatedVideoMemoryBytes(SizeGPU);
}

void UStreamableSparseVolumeTexture::GetAssetRegistryTags(TArray<FAssetRegistryTag>& OutTags) const
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS;
	Super::GetAssetRegistryTags(OutTags);
	PRAGMA_ENABLE_DEPRECATION_WARNINGS;
}

void UStreamableSparseVolumeTexture::GetAssetRegistryTags(FAssetRegistryTagsContext Context) const
{
	Super::GetAssetRegistryTags(Context);

#if WITH_EDITORONLY_DATA
	if (AssetImportData)
	{
		Context.AddTag(FAssetRegistryTag(SourceFileTagName(), AssetImportData->GetSourceData().ToJson(), FAssetRegistryTag::TT_Hidden));
	}
#endif
}

void UStreamableSparseVolumeTexture::AddAssetUserData(UAssetUserData* InUserData)
{
	if (InUserData != nullptr)
	{
		UAssetUserData* ExistingData = GetAssetUserDataOfClass(InUserData->GetClass());
		if (ExistingData != nullptr)
		{
			AssetUserData.Remove(ExistingData);
		}
		AssetUserData.Add(InUserData);
	}
}

UAssetUserData* UStreamableSparseVolumeTexture::GetAssetUserDataOfClass(TSubclassOf<UAssetUserData> InUserDataClass)
{
	for (UAssetUserData* Datum : AssetUserData)
	{
		if (Datum != nullptr && Datum->IsA(InUserDataClass))
		{
			return Datum;
		}
	}
	return nullptr;
}

void UStreamableSparseVolumeTexture::RemoveUserDataOfClass(TSubclassOf<UAssetUserData> InUserDataClass)
{
	for (int32 DataIdx = 0; DataIdx < AssetUserData.Num(); DataIdx++)
	{
		UAssetUserData* Datum = AssetUserData[DataIdx];
		if (Datum != nullptr && Datum->IsA(InUserDataClass))
		{
			AssetUserData.RemoveAt(DataIdx);
			return;
		}
	}
}

const TArray<UAssetUserData*>* UStreamableSparseVolumeTexture::GetAssetUserDataArray() const
{
	return &ToRawPtrTArrayUnsafe(AssetUserData);
}

#if WITH_EDITOR
void UStreamableSparseVolumeTexture::OnAssetsAddExtraObjectsToDelete(TArray<UObject*>& ObjectsToDelete)
{
	if (ObjectsToDelete.Contains(this))
	{
		// When UStreamableSparseVolumeTexture is deleted, we also want all owned USparseVolumeTextureFrame objects to be deleted.
		for (USparseVolumeTextureFrame* Frame : Frames)
		{
			if (Frame)
			{
				ObjectsToDelete.Add(Frame);
			}
		}
	}
}

void UStreamableSparseVolumeTexture::OnAddExtraObjectsToDelete(const TArray<UObject*>& InObjectsToDelete, TSet<UObject*>& OutSecondaryObjects)
{
	if (InObjectsToDelete.Contains(this))
	{
		// When UStreamableSparseVolumeTexture is deleted, we also want all owned USparseVolumeTextureFrame objects to be deleted.
		for (USparseVolumeTextureFrame* Frame : Frames)
		{
			if (Frame)
			{
				OutSecondaryObjects.Add(Frame);
			}
		}
	}
	
}
#endif

#if WITH_EDITORONLY_DATA
void UStreamableSparseVolumeTexture::RecacheFrames()
{
	if (IsTemplate())
	{
		return;
	}

	if (InitState != EInitState_Done)
	{
		UE_LOGF(LogSparseVolumeTexture, Warning, "Tried to cache derived data of an uninitialized SVT: %ls", *GetName());
		return;
	}

	FScopedSlowTask RecacheTask(static_cast<float>(Frames.Num() + 2), LOCTEXT("SparseVolumeTextureCacheFrames", "Caching SparseVolumeTexture frames in Derived Data Cache"));
	RecacheTask.MakeDialog(true);

	const bool bCanEverRender = FApp::CanEverRender();
	if (bCanEverRender)
	{
		UE::SVT::GetStreamingManager().Remove_GameThread(this);
	}
	
	RecacheTask.EnterProgressFrame(1.0f);

	bool bCanceled = false;
	for (USparseVolumeTextureFrame* Frame : Frames)
	{
		check(Frame); // RecacheFrames() is assumed to never be called when the Frames array is invalid (has nullptr elements). Elements may be nulled as part of deleting the SVT.
		if (!bCanceled && RecacheTask.ShouldCancel())
		{
			bCanceled = true;
			UE_LOGF(LogSparseVolumeTexture, Warning, "Canceled '%ls' SparseVolumeTexture caching. All frames not yet cached will be empty!", *GetName());
		}
		if (!bCanceled)
		{
			RecacheTask.EnterProgressFrame(1.0f);
		}
		
		// Even when the user cancels the caching, we still need valid data, so skip any DDC interactions and any actual building and instead just set the FResources
		// of the frame to (valid) empty/default data.
		const bool bSkipDDCAndSetResourcesToDefault = bCanceled;
		Frame->Cache(bSkipDDCAndSetResourcesToDefault);
	}

	if (!bCanceled)
	{
		RecacheTask.EnterProgressFrame(1.0f);
	}
	
	if (bCanEverRender)
	{
		UE::SVT::GetStreamingManager().Add_GameThread(this);
	}
}
#endif

#if WITH_EDITOR
bool UStreamableSparseVolumeTexture::ShouldRegisterDelegates()
{
	return GEditor && !IsTemplate() && !IsRunningCookCommandlet();
}
void UStreamableSparseVolumeTexture::RegisterEditorDelegates()
{
	if (ShouldRegisterDelegates())
	{
		UnregisterEditorDelegates();
		FEditorDelegates::OnAddExtraObjectsToDelete.AddUObject(this, &UStreamableSparseVolumeTexture::OnAddExtraObjectsToDelete);
	}
}
void UStreamableSparseVolumeTexture::UnregisterEditorDelegates()
{
	if (ShouldRegisterDelegates())
	{
		FEditorDelegates::OnAddExtraObjectsToDelete.RemoveAll(this);
	}
}
#endif // WITH_EDITOR

////////////////////////////////////////////////////////////////////////////////////////////////

UStaticSparseVolumeTexture::UStaticSparseVolumeTexture(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

bool UStaticSparseVolumeTexture::AppendFrame(UE::SVT::FTextureData& UncookedFrame, const FTransform& InFrameTransform)
{
	if (!Frames.IsEmpty())
	{
		UE_LOGF(LogSparseVolumeTexture, Error, "Tried to initialize a UStaticSparseVolumeTexture with more than 1 frame");
		return false;
	}
	return Super::AppendFrame(UncookedFrame, InFrameTransform);
}

////////////////////////////////////////////////////////////////////////////////////////////////

UAnimatedSparseVolumeTexture::UAnimatedSparseVolumeTexture(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

////////////////////////////////////////////////////////////////////////////////////////////////

UAnimatedSparseVolumeTextureController::UAnimatedSparseVolumeTextureController(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

void UAnimatedSparseVolumeTextureController::Play()
{
	bIsPlaying = true;
}

void UAnimatedSparseVolumeTextureController::Pause()
{
	bIsPlaying = false;
}

void UAnimatedSparseVolumeTextureController::Stop()
{
	if (bIsPlaying)
	{
		bIsPlaying = false;
		Time = 0.0f;
	}
}

void UAnimatedSparseVolumeTextureController::Update(float DeltaTime)
{
	if (!IsValid(SparseVolumeTexture) || !bIsPlaying)
	{
		return;
	}

	// Update animation time
	const float AnimationDuration = GetDuration();
	Time = FMath::Fmod(Time + DeltaTime, AnimationDuration + UE_SMALL_NUMBER);
}

float UAnimatedSparseVolumeTextureController::GetFractionalFrameIndex()
{
	if (!IsValid(SparseVolumeTexture))
	{
		return 0.0f;
	}

	const int32 FrameCount = SparseVolumeTexture->GetNumFrames();
	const float FrameIndexF = FMath::Fmod(Time * FrameRate, (float)FrameCount);
	return FrameIndexF;
}

USparseVolumeTextureFrame* UAnimatedSparseVolumeTextureController::GetFrameByIndex(int32 FrameIndex)
{
	if (!IsValid(SparseVolumeTexture))
	{
		return nullptr;
	}

	return USparseVolumeTextureFrame::GetFrameAndIssueStreamingRequest(SparseVolumeTexture, GetTypeHash(this), 0.0f /*FrameRate*/, FrameIndex, MipLevel, bBlockingStreamingRequests, false /*bHasValidFrameRate*/);
}

USparseVolumeTextureFrame* UAnimatedSparseVolumeTextureController::GetCurrentFrame()
{
	if (!IsValid(SparseVolumeTexture))
	{
		return nullptr;
	}

	// Compute (fractional) index of frame to sample
	const float FrameIndexF = GetFractionalFrameIndex();

	return USparseVolumeTextureFrame::GetFrameAndIssueStreamingRequest(SparseVolumeTexture, GetTypeHash(this), FrameRate, FrameIndexF, MipLevel, bBlockingStreamingRequests, true /*bHasValidFrameRate*/);
}

void UAnimatedSparseVolumeTextureController::GetCurrentFramesForInterpolation(USparseVolumeTextureFrame*& Frame0, USparseVolumeTextureFrame*& Frame1, float& LerpAlpha)
{
	if (!IsValid(SparseVolumeTexture))
	{
		return;
	}

	// Compute (fractional) index of frame to sample
	const float FrameIndexF = GetFractionalFrameIndex();
	const int32 FrameIndex = (int32)FrameIndexF;
	LerpAlpha = FMath::Frac(FrameIndexF);

	const uint32 StreamingInstanceKey = GetTypeHash(this);
	Frame0 = USparseVolumeTextureFrame::GetFrameAndIssueStreamingRequest(SparseVolumeTexture, StreamingInstanceKey, FrameRate, FrameIndexF, MipLevel, bBlockingStreamingRequests, true /*bHasValidFrameRate*/);
	Frame1 = USparseVolumeTextureFrame::GetFrameAndIssueStreamingRequest(SparseVolumeTexture, StreamingInstanceKey, FrameRate, (FrameIndex + 1) % SparseVolumeTexture->GetNumFrames(), MipLevel, bBlockingStreamingRequests, true /*bHasValidFrameRate*/);
}

float UAnimatedSparseVolumeTextureController::GetDuration()
{
	if (!IsValid(SparseVolumeTexture))
	{
		return 0.0f;
	}

	const int32 FrameCount = SparseVolumeTexture->GetNumFrames();
	const float AnimationDuration = FrameCount / (FrameRate + UE_SMALL_NUMBER);
	return AnimationDuration;
}

////////////////////////////////////////////////////////////////////////////////////////////////

#undef LOCTEXT_NAMESPACE
