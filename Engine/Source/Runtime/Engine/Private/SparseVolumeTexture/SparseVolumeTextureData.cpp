// Copyright Epic Games, Inc. All Rights Reserved.

#include "SparseVolumeTexture/SparseVolumeTextureData.h"
#include "SparseVolumeTexture/SparseVolumeTextureUtility.h"
#include "Async/ParallelFor.h"
#include "Hash/CityHash.h"
#include <atomic>

DEFINE_LOG_CATEGORY_STATIC(LogSparseVolumeTextureData, Log, All);

// Enabling this ensures proper bilinear filtering between physical pages and empty pages by tagging neighboring empty pages as resident/physical. This causes more physical tiles to be generated though.
#define SVT_CORRECT_TILE_ALLOCATION_FOR_LINEAR_FILTERING 0

// Enabling this ensures that the order of the physical tiles will be deterministic.
#define SVT_DETERMINISTIC_PAGE_TABLE_GENERATION 1

////////////////////////////////////////////////////////////////////////////////////////////////

FArchive& operator<<(FArchive& Ar, UE::SVT::FTextureData::FMipMap& MipMap)
{
	Ar << MipMap.PageTable;
	Ar << MipMap.PhysicalTileDataA;
	Ar << MipMap.PhysicalTileDataB;
	Ar << MipMap.NumPhysicalTiles;
	return Ar;
}

FArchive& operator<<(FArchive& Ar, UE::SVT::FTextureData& TextureData)
{
	Ar << TextureData.Header;
	Ar << TextureData.FallbackValuesQuantized;
	Ar << TextureData.MipMaps;
	return Ar;
}

////////////////////////////////////////////////////////////////////////////////////////////////

namespace UE
{
namespace SVT
{

bool FTextureData::Create(const ITextureDataProvider& DataProvider)
{
	const FTextureDataCreateInfo CreateInfo = DataProvider.GetCreateInfo();

	Header = FHeader(CreateInfo.VirtualVolumeAABBMin, CreateInfo.VirtualVolumeAABBMax, CreateInfo.AttributesFormats[0], CreateInfo.AttributesFormats[1], CreateInfo.FallbackValues[0], CreateInfo.FallbackValues[1]);
	if (!Header.Validate(true /*bPrintToLog*/))
	{
		return false;
	}

	const int32 FormatSize[] = { GPixelFormats[Header.AttributesFormats[0]].BlockBytes, GPixelFormats[Header.AttributesFormats[1]].BlockBytes };
	uint8 NullTileValuesU8[2][sizeof(float) * 4] = {};
	for (int32 i = 0; i < 2; ++i)
	{
		if (Header.AttributesFormats[i] != PF_Unknown)
		{
			SVT::WriteVoxel(0, NullTileValuesU8[i], Header.AttributesFormats[i], Header.FallbackValues[i]);
			FallbackValuesQuantized[i] = SVT::ReadVoxel(0, NullTileValuesU8[i], Header.AttributesFormats[i]);
		}
	}

	MipMaps.SetNum(1);

	// Tag all pages with valid data
	TBitArray PagesWithData(false, Header.PageTableVolumeResolution.Z * Header.PageTableVolumeResolution.Y * Header.PageTableVolumeResolution.X);
	DataProvider.IteratePhysicalSource([&](const FIntVector3& Coord, int32 AttributesIdx, int32 ComponentIdx, float VoxelValue)
		{
			const bool bIsFallbackValue = (VoxelValue == Header.FallbackValues[AttributesIdx][ComponentIdx]);
			if (!bIsFallbackValue)
			{
				check(IsInBounds(Coord, Header.VirtualVolumeAABBMin, Header.VirtualVolumeAABBMax));

#if SVT_CORRECT_TILE_ALLOCATION_FOR_LINEAR_FILTERING
				// Tag all pages touching the 3x3 area around this voxel in order to properly support bilinear/border voxels
				for (int32 OffsetZ = -1; OffsetZ < 2; ++OffsetZ)
				{
					for (int32 OffsetY = -1; OffsetY < 2; ++OffsetY)
					{
						for (int32 OffsetX = -1; OffsetX < 2; ++OffsetX)
						{
							const FIntVector3 GridCoord = Coord + FIntVector3(OffsetX, OffsetY, OffsetZ);
							if (IsInBounds(GridCoord, Header.VirtualVolumeAABBMin, Header.VirtualVolumeAABBMax))
							{
								const FIntVector3 PageCoord = (GridCoord / SPARSE_VOLUME_TILE_RES) - Header.PageTableVolumeAABBMin;
								check(IsInBounds(PageCoord, FIntVector3::ZeroValue, Header.PageTableVolumeResolution));
								const int32 PageIndex = PageCoord.Z * (Header.PageTableVolumeResolution.Y * Header.PageTableVolumeResolution.X) + PageCoord.Y * Header.PageTableVolumeResolution.X + PageCoord.X;
								PagesWithData[PageIndex] = true;
							}
						}
					}
				}
#else // SVT_CORRECT_TILE_ALLOCATION_FOR_LINEAR_FILTERING
				const FIntVector3 GridCoord = Coord;
				const FIntVector3 PageCoord = (GridCoord / SPARSE_VOLUME_TILE_RES) - Header.PageTableVolumeAABBMin;
				check(IsInBounds(PageCoord, FIntVector3::ZeroValue, Header.PageTableVolumeResolution));
				const int32 PageIndex = PageCoord.Z * (Header.PageTableVolumeResolution.Y * Header.PageTableVolumeResolution.X) + PageCoord.Y * Header.PageTableVolumeResolution.X + PageCoord.X;
				PagesWithData[PageIndex] = true;
#endif // SVT_CORRECT_TILE_ALLOCATION_FOR_LINEAR_FILTERING
			}
		});

	// Initialize the page table and tile data arrays.
	MipMaps[0].PageTable.SetNumZeroed(Header.PageTableVolumeResolution.X * Header.PageTableVolumeResolution.Y * Header.PageTableVolumeResolution.Z);

	int32 NumAllocatedPages = 0;
	for (TConstSetBitIterator It(PagesWithData); It; ++It)
	{
		const int32 PageIndex = It.GetIndex();
		MipMaps[0].PageTable[PageIndex] = (uint32)++NumAllocatedPages;
	}

	MipMaps[0].PhysicalTileDataA.SetNum((int64)NumAllocatedPages * (int64)SVT::NumVoxelsPerPaddedTile * (int64)FormatSize[0]);
	MipMaps[0].PhysicalTileDataB.SetNum((int64)NumAllocatedPages * (int64)SVT::NumVoxelsPerPaddedTile * (int64)FormatSize[1]);
	MipMaps[0].NumPhysicalTiles = NumAllocatedPages;

	// Fill all tiles with fallback values/null tile values. We can't guarantee that all voxels of a page will be written to in the next step, so if we don't do this,
	// we might end up with physical tiles that have only been partially filled with valid voxels.
	ParallelFor(NumAllocatedPages, [&](int32 TileIndex)
		{
			uint8* TileData[2] = { MipMaps[0].PhysicalTileDataA.GetData(), MipMaps[0].PhysicalTileDataB.GetData() };
			for (int32 i = 0; i < 2; ++i)
			{
				if (FormatSize[i])
				{
					for (int32 VoxelIndex = 0; VoxelIndex < SVT::NumVoxelsPerPaddedTile; ++VoxelIndex)
					{
						const int64 VoxelIndexGlobal = (int64)TileIndex * (int64)SVT::NumVoxelsPerPaddedTile + (int64)VoxelIndex;
						FMemory::Memcpy(TileData[i] + VoxelIndexGlobal * FormatSize[i], NullTileValuesU8[i], FormatSize[i]);
					}
				}
			}
		});

	// Write physical tile data
	DataProvider.IteratePhysicalSource([&](const FIntVector3& Coord, int32 AttributesIdx, int32 ComponentIdx, float VoxelValue)
		{
			const FIntVector3 GridCoord = Coord;
			check(IsInBounds(GridCoord, Header.VirtualVolumeAABBMin, Header.VirtualVolumeAABBMax));
			const FIntVector3 PageCoord = (GridCoord / SPARSE_VOLUME_TILE_RES) - Header.PageTableVolumeAABBMin;
			check(IsInBounds(PageCoord, FIntVector3::ZeroValue, Header.PageTableVolumeResolution));
			const int32 PageIndex = PageCoord.Z * (Header.PageTableVolumeResolution.Y * Header.PageTableVolumeResolution.X) + PageCoord.Y * Header.PageTableVolumeResolution.X + PageCoord.X;

			const uint32 PageEntry = MipMaps[0].PageTable[PageIndex];
			if (PageEntry != 0)
			{
				const int32 TileIndex = (int32)PageEntry - 1; // -1 because 0 is reserved for the null tile which we don't actually store in memory
				const FIntVector3 TileLocalCoord = GridCoord % SPARSE_VOLUME_TILE_RES;
				WriteTileDataVoxel(TileIndex, TileLocalCoord, 0 /*MipLevel*/, AttributesIdx, FVector4f(VoxelValue, VoxelValue, VoxelValue, VoxelValue), ComponentIdx);
			}
		});

	return true;
}

bool FTextureData::CreateFromDense(const FTextureDataCreateInfo& CreateInfo, const TArrayView64<uint8>& VoxelDataA, const TArrayView64<uint8>& VoxelDataB)
{	
	Header = FHeader(CreateInfo.VirtualVolumeAABBMin, CreateInfo.VirtualVolumeAABBMax, CreateInfo.AttributesFormats[0], CreateInfo.AttributesFormats[1], CreateInfo.FallbackValues[0], CreateInfo.FallbackValues[1]);
	if (!Header.Validate(true /*bPrintToLog*/))
	{
		return false;
	}
	
	const int64 FormatSize[] = { GPixelFormats[Header.AttributesFormats[0]].BlockBytes, GPixelFormats[Header.AttributesFormats[1]].BlockBytes };
	const FIntVector3 FractionalPageOffset = Header.VirtualVolumeAABBMin % SPARSE_VOLUME_TILE_RES; // Offset from page origin to source data volume origin
	uint8 NullTileValuesU8[2][sizeof(float) * 4] = {};
	for (int32 i = 0; i < 2; ++i)
	{
		if (Header.AttributesFormats[i] != PF_Unknown)
		{
			SVT::WriteVoxel(0, NullTileValuesU8[i], Header.AttributesFormats[i], Header.FallbackValues[i]);
			FallbackValuesQuantized[i] = SVT::ReadVoxel(0, NullTileValuesU8[i], Header.AttributesFormats[i]);
		}
	}

	MipMaps.SetNum(1);

	MipMaps[0].PageTable.SetNumZeroed(Header.PageTableVolumeResolution.X * Header.PageTableVolumeResolution.Y * Header.PageTableVolumeResolution.Z);
	std::atomic<uint32> NumAllocatedPagesAtomic = 0;

	// This is a two-pass algorithm; first marking pages and allocating physical tiles and then in a second pass writing to the physical tiles.
	// Most of the setup of these two passes is very similar, so we use a single lambda for them.
	auto ProcessBlock = [&](int32 JobIndex, bool bWriteTileData)
	{
		// Make sure we only process voxels inside a single page in the Y and Z dimension so we don't end up reading/writing the same entries in the page table from multiple threads.
		const FIntVector3 PageBegin = FIntVector3(0, JobIndex % Header.PageTableVolumeResolution.Y, JobIndex / Header.PageTableVolumeResolution.Y) + Header.PageTableVolumeAABBMin;
		// PageTableVolumeAABBMax is padded to a power of 2 which may be larger than the tightest possible page table for the source volume.
		// We only want to iterate over pages overlapping the source volume, so we recompute a tight fit here.
		const int32 PageEndX = (Header.VirtualVolumeAABBMax.X + (SPARSE_VOLUME_TILE_RES - 1)) / SPARSE_VOLUME_TILE_RES;
		const FIntVector3 PageEnd = FIntVector3(PageEndX, FMath::Min(PageBegin.Y + 1, Header.PageTableVolumeAABBMax.Y), FMath::Min(PageBegin.Z + 1, Header.PageTableVolumeAABBMax.Z));
		const FIntVector3 PageVoxelBegin = PageBegin * SPARSE_VOLUME_TILE_RES;
		const FIntVector3 PageVoxelEnd = PageEnd * SPARSE_VOLUME_TILE_RES;
		const FIntVector3 ActiveVoxelBegin = PageVoxelBegin + FractionalPageOffset;
		const FIntVector3 ActiveVoxelEnd = FIntVector3(FMath::Min(PageVoxelEnd.X, Header.VirtualVolumeAABBMax.X), FMath::Min(PageVoxelEnd.Y, Header.VirtualVolumeAABBMax.Y), FMath::Min(PageVoxelEnd.Z, Header.VirtualVolumeAABBMax.Z));

		// Iterate over voxels in the YZ plane and over pages along the X axis.
		// For both page table filling and tile writing, we can process an entire page length worth of voxels (along the X axis) at a time.
		for (int64 Z = ActiveVoxelBegin.Z; Z < ActiveVoxelEnd.Z; ++Z)
		{
			for (int64 Y = ActiveVoxelBegin.Y; Y < ActiveVoxelEnd.Y; ++Y)
			{
				for (int64 PageX = PageBegin.X; PageX < PageEnd.X; ++PageX)
				{
					const FIntVector3 GridCoord = FIntVector3(PageX * SPARSE_VOLUME_TILE_RES + FractionalPageOffset.X, Y, Z); // Global/absolute voxel coordinate
					check(IsInBounds(GridCoord, FIntVector3::ZeroValue, Header.VirtualVolumeAABBMax));
					const FIntVector3 PageCoord = (GridCoord / SPARSE_VOLUME_TILE_RES) - Header.PageTableVolumeAABBMin;
					check(IsInBounds(PageCoord, FIntVector3::ZeroValue, Header.PageTableVolumeResolution));
					const int32 PageIndex = PageCoord.Z * (Header.PageTableVolumeResolution.Y * Header.PageTableVolumeResolution.X) + PageCoord.Y * Header.PageTableVolumeResolution.X + PageCoord.X;
					const uint32 TileIndex = MipMaps[0].PageTable[PageIndex];

					// Only write tile data if a physical tile has been allocated.
					// Only try to allocate a physical tile if there is none already.
					if ((bWriteTileData && TileIndex != 0) || (!bWriteTileData && TileIndex == 0))
					{
						// Coordinates into the dense source volume data
						const int64 BeginSrcX = PageX * SPARSE_VOLUME_TILE_RES - Header.VirtualVolumeAABBMin.X;
						const int64 EndSrcX = FMath::Min(BeginSrcX + SPARSE_VOLUME_TILE_RES, Header.VirtualVolumeResolution.X);
						const int64 SrcY = Y - Header.VirtualVolumeAABBMin.Y;
						const int64 SrcZ = Z - Header.VirtualVolumeAABBMin.Z;
						const int64 NumVoxelsToProcess = EndSrcX - BeginSrcX;

						// Compute flat source and destination voxel index. Make sure to account for the fact that the voxels in the dst buffer have padding/a border.
						const FInt64Vector3 LocalCoordPadded = FInt64Vector3(GridCoord % SPARSE_VOLUME_TILE_RES) + FInt64Vector3(SPARSE_VOLUME_TILE_BORDER);
						const int64 DstVoxelIndex = ((int64)(TileIndex - 1) * (int64)SVT::NumVoxelsPerPaddedTile) 
							+ (LocalCoordPadded.Z * (SPARSE_VOLUME_TILE_RES_PADDED * SPARSE_VOLUME_TILE_RES_PADDED))
							+ (LocalCoordPadded.Y * SPARSE_VOLUME_TILE_RES_PADDED)
							+ LocalCoordPadded.X;
						const int64 SrcVoxelIndex = (SrcZ * (Header.VirtualVolumeResolution.Y * Header.VirtualVolumeResolution.X)) + (SrcY * Header.VirtualVolumeResolution.X) + BeginSrcX;
						
						if (bWriteTileData)
						{
							// Copy the whole row at once!
							if (FormatSize[0])
							{
								FMemory::Memcpy(MipMaps[0].PhysicalTileDataA.GetData() + DstVoxelIndex * FormatSize[0], &VoxelDataA[SrcVoxelIndex * FormatSize[0]], FormatSize[0] * NumVoxelsToProcess);
							}
							if (FormatSize[1])
							{
								FMemory::Memcpy(MipMaps[0].PhysicalTileDataB.GetData() + DstVoxelIndex * FormatSize[1], &VoxelDataB[SrcVoxelIndex * FormatSize[1]], FormatSize[1] * NumVoxelsToProcess);
							}
						}
						else
						{
							// Compare each voxel of the row against the quantized fallback value and only allocate a physical tile if there are non-fallback voxels present.
							bool bHasNonFallbackValue = false;
							for (int64 i = 0; i < NumVoxelsToProcess && !bHasNonFallbackValue; ++i)
							{
								bHasNonFallbackValue |= (FormatSize[0] != 0) && (FMemory::Memcmp(&VoxelDataA[(SrcVoxelIndex + i) * FormatSize[0]], NullTileValuesU8[0], FormatSize[0]) != 0);
								bHasNonFallbackValue |= (FormatSize[1] != 0) && (FMemory::Memcmp(&VoxelDataB[(SrcVoxelIndex + i) * FormatSize[1]], NullTileValuesU8[1], FormatSize[1]) != 0);
							}
							if (bHasNonFallbackValue)
							{
								MipMaps[0].PageTable[PageIndex] = NumAllocatedPagesAtomic.fetch_add(1) + 1u;
							}
						}
					}
				}
			}
		}
	};

	// Process an entire row of pages along the X dimension per job. We iterate over the logical pages and sample the dense source voxel data; we do not iterate over the physical tiles in the destination SVT.
	const int32 NumPageTableJobs = Header.PageTableVolumeResolution.Z * Header.PageTableVolumeResolution.Y;

	// Create page table
	ParallelFor(NumPageTableJobs, [&](int32 Index) { ProcessBlock(Index, false /*bWriteTileData*/); });
	const int32 NumAllocatedPages = (int32)NumAllocatedPagesAtomic.load();

#if SVT_DETERMINISTIC_PAGE_TABLE_GENERATION
	{
		uint32 NextPageTableEntry = 1;
		for (uint32& PageTableEntry : MipMaps[0].PageTable)
		{
			if (PageTableEntry != 0)
			{
				PageTableEntry = NextPageTableEntry++;
			}
		}
		check((NumAllocatedPages + 1) == NextPageTableEntry);
	}
#endif // SVT_DETERMINISTIC_PAGE_TABLE_GENERATION

	// Allocate tile data
	MipMaps[0].PhysicalTileDataA.SetNum((int64)NumAllocatedPages * (int64)SVT::NumVoxelsPerPaddedTile * (int64)FormatSize[0]);
	MipMaps[0].PhysicalTileDataB.SetNum((int64)NumAllocatedPages * (int64)SVT::NumVoxelsPerPaddedTile * (int64)FormatSize[1]);
	MipMaps[0].NumPhysicalTiles = NumAllocatedPages;

	// Clear all physical tiles to the fallback value. We can't guarantee that every voxel will be written to in the next step, so we need to ensure that tiles are always fully written to with valid data.
	// SVT_TODO: Make this smarter. We don't need to write ALL voxels in this stage because we know which voxels will be overwritten anyways.
	if (FractionalPageOffset != FIntVector3::ZeroValue || (Header.VirtualVolumeAABBMax % SPARSE_VOLUME_TILE_RES) != FIntVector3::ZeroValue)
	{
		// Iterate over physical tiles (which at this point equals the number of allocated pages); NOT logical pages!
		ParallelFor(NumAllocatedPages, [&](int32 TileIndex)
			{
				uint8* TileData[2] = { MipMaps[0].PhysicalTileDataA.GetData(), MipMaps[0].PhysicalTileDataB.GetData() };
				for (int32 i = 0; i < 2; ++i)
				{
					if (FormatSize[i])
					{
						for (int32 VoxelIndex = 0; VoxelIndex < SVT::NumVoxelsPerPaddedTile; ++VoxelIndex)
						{
							const int64 VoxelIndexGlobal = (int64)TileIndex * (int64)SVT::NumVoxelsPerPaddedTile + (int64)VoxelIndex;
							FMemory::Memcpy(TileData[i] + VoxelIndexGlobal * FormatSize[i], NullTileValuesU8[i], FormatSize[i]);
						}
					}
				}
			});
	}

	// Write physical tile data
	ParallelFor(NumPageTableJobs, [&](int32 Index) { ProcessBlock(Index, true /*bWriteTileData*/); });

	return true;
}

void FTextureData::CreateDefault()
{
	// Store a single 1x1x1 mip level with the page table pointing to a (zero-valued) null tile.
	Header = FHeader(FIntVector3::ZeroValue, FIntVector3(1, 1, 1), PF_R8, PF_Unknown, FVector4f(), FVector4f());

	MipMaps.SetNum(1);
	MipMaps[0].PageTable.SetNumZeroed(1);
	MipMaps[0].PhysicalTileDataA.Empty();
	MipMaps[0].PhysicalTileDataB.Empty();
	MipMaps[0].NumPhysicalTiles = 0;
}

uint32 FTextureData::ReadPageTable(const FIntVector3& PageTableCoord, int32 MipLevel) const
{
	if (!MipMaps.IsValidIndex(MipLevel))
	{
		return INDEX_NONE;
	}
	const FIntVector3 PageTableAABBMipMin = Header.PageTableVolumeAABBMin >> MipLevel;
	const FIntVector3 PageTableAABBMipMax = ShiftRightAndMax(Header.PageTableVolumeAABBMax, MipLevel, 1);
	if (!IsInBounds(PageTableCoord, PageTableAABBMipMin, PageTableAABBMipMax))
	{
		return INDEX_NONE;
	}

	const FIntVector3 PageTableResolution = PageTableAABBMipMax - PageTableAABBMipMin;
	check(PageTableResolution == ShiftRightAndMax(Header.PageTableVolumeResolution, MipLevel, 1));
	const FIntVector3 LocalPageTableCoord = PageTableCoord - PageTableAABBMipMin; // Local to the AABB of the page table
	const int32 PageIndex = LocalPageTableCoord.Z * (PageTableResolution.Y * PageTableResolution.X) + LocalPageTableCoord.Y * PageTableResolution.X + LocalPageTableCoord.X;
	if (MipMaps[MipLevel].PageTable.IsValidIndex(PageIndex))
	{
		return MipMaps[MipLevel].PageTable[PageIndex];
	}
	return INDEX_NONE;
}

FVector4f FTextureData::ReadTileDataVoxel(int32 TileIndex, const FIntVector3& TileDataCoord, int32 MipLevel, int32 AttributesIdx) const
{
	check(AttributesIdx >= 0 && AttributesIdx <= 1);

	if (!MipMaps.IsValidIndex(MipLevel)
		|| !IsInBounds(TileDataCoord, FIntVector3::ZeroValue, FIntVector3(SPARSE_VOLUME_TILE_RES))
		|| TileIndex < 0 || TileIndex >= MipMaps[MipLevel].NumPhysicalTiles
		|| (AttributesIdx == 0 && MipMaps[MipLevel].PhysicalTileDataA.IsEmpty()) 
		|| (AttributesIdx == 1 && MipMaps[MipLevel].PhysicalTileDataB.IsEmpty()))
	{
		return FallbackValuesQuantized[AttributesIdx];
	}

	const EPixelFormat Format = Header.AttributesFormats[AttributesIdx];

	// Sanity check to make sure that the tile data offset is within bounds
	const int64 TileByteOffset = (int64)TileIndex * (int64)SVT::NumVoxelsPerPaddedTile * (int64)GPixelFormats[Format].BlockBytes;
	const int64 NumTileBytes = AttributesIdx == 0 ? MipMaps[MipLevel].PhysicalTileDataA.Num() : MipMaps[MipLevel].PhysicalTileDataB.Num();
	check(TileByteOffset < NumTileBytes);

	const FInt64Vector3 CoordPadded = FInt64Vector3(TileDataCoord) + FInt64Vector3(SPARSE_VOLUME_TILE_BORDER);
	const int64 VoxelIndex = (int64)TileIndex * (int64)SVT::NumVoxelsPerPaddedTile + CoordPadded.Z * (SPARSE_VOLUME_TILE_RES_PADDED * SPARSE_VOLUME_TILE_RES_PADDED) + CoordPadded.Y * SPARSE_VOLUME_TILE_RES_PADDED + CoordPadded.X;
	const uint8* TileData = AttributesIdx == 0 ? MipMaps[MipLevel].PhysicalTileDataA.GetData() : MipMaps[MipLevel].PhysicalTileDataB.GetData();
	return SVT::ReadVoxel(VoxelIndex, TileData, Format);
}

FVector4f FTextureData::Load(const FIntVector3& VolumeCoord, int32 MipLevel, int32 AttributesIdx, const FTextureDataAddressingInfo& AddressingInfo) const
{
	// Out of bounds VolumeCoord or MipLevel is caught by ReadPageTable() and handled by returning INDEX_NONE
	check(AttributesIdx >= 0 && AttributesIdx <= 1);

	auto ApplyAddressMode = [](int32 x, int32 Width, TextureAddress Mode)
	{
		switch (Mode)
		{
		case TA_Wrap:
			return (x % Width + Width) % Width; // Make sure it's a proper modulo for negative numbers ....
		case TA_Clamp:
			return FMath::Clamp(x, 0, Width - 1);
		case TA_Mirror:
			int32 DoubleWidth = Width + Width;
			int32 DoubleWrap = (x % DoubleWidth + DoubleWidth) % DoubleWidth;
			return (DoubleWrap < Width) ? DoubleWrap : (Width - 1) - (DoubleWrap - Width);
		}
		return x;
	};
	const FIntVector3 AddressModeVolumeCoord = FIntVector3(
		ApplyAddressMode(VolumeCoord.X, FMath::Max(1, AddressingInfo.VolumeResolution.X >> MipLevel), AddressingInfo.AddressX),
		ApplyAddressMode(VolumeCoord.Y, FMath::Max(1, AddressingInfo.VolumeResolution.Y >> MipLevel), AddressingInfo.AddressY),
		ApplyAddressMode(VolumeCoord.Z, FMath::Max(1, AddressingInfo.VolumeResolution.Z >> MipLevel), AddressingInfo.AddressZ));
	const FIntVector3 PageTableCoord = AddressModeVolumeCoord / SPARSE_VOLUME_TILE_RES;
	const uint32 PageTableEntry = ReadPageTable(PageTableCoord, MipLevel);
	if (PageTableEntry == INDEX_NONE || PageTableEntry == 0)
	{
		return FallbackValuesQuantized[AttributesIdx];
	}
	check(PageTableEntry > 0);
	const int32 TileIndex = (int32)PageTableEntry - 1; // -1 because we don't store the null tile in memory.
	const FIntVector3 TileLocalCoord = AddressModeVolumeCoord % SPARSE_VOLUME_TILE_RES;
	const FVector4f Sample = ReadTileDataVoxel(TileIndex, TileLocalCoord, MipLevel, AttributesIdx);
	return Sample;
}

void FTextureData::WriteTileDataVoxel(int32 TileIndex, const FIntVector3& TileDataCoord, int32 MipLevel, int32 AttributesIdx, const FVector4f& Value, int32 DstComponent)
{
	check(AttributesIdx >= 0 && AttributesIdx <= 1);
	
	// Allow TileDataCoord to extend past SPARSE_VOLUME_TILE_RES by SPARSE_VOLUME_TILE_BORDER.
	// This allows us to write the border voxels without changing how TileDataCoord needs to be computed for other use cases.
	if (!IsInBounds(TileDataCoord, FIntVector3(-SPARSE_VOLUME_TILE_BORDER), FIntVector3(SPARSE_VOLUME_TILE_RES + SPARSE_VOLUME_TILE_BORDER))
		|| !MipMaps.IsValidIndex(MipLevel)
		|| TileIndex < 0 || TileIndex >= MipMaps[MipLevel].NumPhysicalTiles
		|| (AttributesIdx == 0 && MipMaps[MipLevel].PhysicalTileDataA.IsEmpty())
		|| (AttributesIdx == 1 && MipMaps[MipLevel].PhysicalTileDataB.IsEmpty()))
	{
		return;
	}

	const FInt64Vector3 CoordPadded = FInt64Vector3(TileDataCoord) + FInt64Vector3(SPARSE_VOLUME_TILE_BORDER);
	const int64 VoxelIndex = (int64)TileIndex * (int64)SVT::NumVoxelsPerPaddedTile + CoordPadded.Z * (SPARSE_VOLUME_TILE_RES_PADDED * SPARSE_VOLUME_TILE_RES_PADDED) + CoordPadded.Y * SPARSE_VOLUME_TILE_RES_PADDED + CoordPadded.X;
	uint8* TileData = AttributesIdx == 0 ? MipMaps[MipLevel].PhysicalTileDataA.GetData() : MipMaps[MipLevel].PhysicalTileDataB.GetData();
	const EPixelFormat Format = Header.AttributesFormats[AttributesIdx];
	SVT::WriteVoxel(VoxelIndex, TileData, Format, Value, DstComponent);
}

bool FTextureData::GenerateMipMaps(const FTextureDataAddressingInfo& AddressingInfo, int32 NumMipLevels)
{
	const int32 FormatSize[] = { GPixelFormats[Header.AttributesFormats[0]].BlockBytes, GPixelFormats[Header.AttributesFormats[1]].BlockBytes };

	check(!MipMaps.IsEmpty());
	check(FMath::IsPowerOfTwo(Header.PageTableVolumeResolution.X)
		&& FMath::IsPowerOfTwo(Header.PageTableVolumeResolution.Y)
		&& FMath::IsPowerOfTwo(Header.PageTableVolumeResolution.Z));
	const int32 NumMip0Tiles = MipMaps[0].NumPhysicalTiles;
	check(MipMaps[0].PhysicalTileDataA.Num() >= ((int64)NumMip0Tiles * (int64)SVT::NumVoxelsPerPaddedTile * (int64)FormatSize[0]));
	check(MipMaps[0].PhysicalTileDataB.Num() >= ((int64)NumMip0Tiles * (int64)SVT::NumVoxelsPerPaddedTile * (int64)FormatSize[1]));
	check(MipMaps[0].PageTable.Num() == (Header.PageTableVolumeResolution.X * Header.PageTableVolumeResolution.Y * Header.PageTableVolumeResolution.Z));
	
	if (NumMipLevels == -1)
	{
		NumMipLevels = 1;
		FIntVector3 PageTableResolution = Header.PageTableVolumeResolution;
		while (PageTableResolution.X > 1 || PageTableResolution.Y > 1 || PageTableResolution.Z > 1)
		{
			PageTableResolution /= 2;
			++NumMipLevels;
		}
	}

	MipMaps.SetNum(1);
	MipMaps[0].PhysicalTileDataA.SetNum((int64)NumMip0Tiles * (int64)SVT::NumVoxelsPerPaddedTile * (int64)FormatSize[0]);
	MipMaps[0].PhysicalTileDataB.SetNum((int64)NumMip0Tiles * (int64)SVT::NumVoxelsPerPaddedTile * (int64)FormatSize[1]);

	for (int32 MipLevel = 1; MipLevel < NumMipLevels; ++MipLevel)
	{
		MipMaps.AddDefaulted();

		const FIntVector3 PageTableVolumeAABBMin = Header.PageTableVolumeAABBMin >> MipLevel;
		const FIntVector3 PageTableVolumeResolution = ShiftRightAndMax(Header.PageTableVolumeResolution, MipLevel, 1);
		check(FMath::IsPowerOfTwo(PageTableVolumeResolution.X)
			&& FMath::IsPowerOfTwo(PageTableVolumeResolution.Y)
			&& FMath::IsPowerOfTwo(PageTableVolumeResolution.Z));

		// Allocate some memory for temp data (worst case)
		TArray<FIntVector3> LinearAllocatedPages;
		LinearAllocatedPages.SetNum(PageTableVolumeResolution.X * PageTableVolumeResolution.Y * PageTableVolumeResolution.Z);

		// Go over each potential page from the source data and push allocate it if it has any data.
		// Otherwise point to the default empty page.
		int32 NumAllocatedPages = 0;
		for (int32 PageZ = 0; PageZ < PageTableVolumeResolution.Z; ++PageZ)
		{
			for (int32 PageY = 0; PageY < PageTableVolumeResolution.Y; ++PageY)
			{
				for (int32 PageX = 0; PageX < PageTableVolumeResolution.X; ++PageX)
				{
					const FIntVector3 PageCoord = FIntVector3(PageX, PageY, PageZ);
					bool bHasAnyData = false;
					for (int32 OffsetIdx = 0; OffsetIdx < 8; ++OffsetIdx)
					{
						const FIntVector3 Offset = FIntVector3(OffsetIdx, OffsetIdx >> 1, OffsetIdx >> 2) & 1;
						const FIntVector3 ParentPageTableCoord = (PageTableVolumeAABBMin + PageCoord) * 2 + Offset;
						const uint32 PageSample = ReadPageTable(ParentPageTableCoord, MipLevel - 1);

						if (PageSample != INDEX_NONE && PageSample != 0)
						{
							bHasAnyData = true;
						}
					}
					if (bHasAnyData)
					{
						LinearAllocatedPages[NumAllocatedPages] = PageCoord;
						NumAllocatedPages++;
					}
				}
			}
		}
		LinearAllocatedPages.SetNum(NumAllocatedPages);

		// Initialize the SparseVolumeTexture page and tile.
		MipMaps[MipLevel].PageTable.SetNumZeroed(PageTableVolumeResolution.X * PageTableVolumeResolution.Y * PageTableVolumeResolution.Z);
		MipMaps[MipLevel].PhysicalTileDataA.SetNum((int64)NumAllocatedPages * (int64)SVT::NumVoxelsPerPaddedTile * (int64)FormatSize[0]);
		MipMaps[MipLevel].PhysicalTileDataB.SetNum((int64)NumAllocatedPages * (int64)SVT::NumVoxelsPerPaddedTile * (int64)FormatSize[1]);
		MipMaps[MipLevel].NumPhysicalTiles = NumAllocatedPages;

		// Generate page table and tile volume data by splatting the data
		{
			ParallelFor(NumAllocatedPages, [this, &LinearAllocatedPages, &PageTableVolumeResolution, &PageTableVolumeAABBMin, &AddressingInfo, MipLevel](int32 PageIndex)
				{
					const FIntVector3 PageCoordToSplat = LinearAllocatedPages[PageIndex];
					const uint32 DstTileIndex = PageIndex + 1; // Page table entries are relative to the mip level, but 0 is reserved for null tile

					// Setup the page table entry
					MipMaps[MipLevel].PageTable
						[
							PageCoordToSplat.Z * PageTableVolumeResolution.X * PageTableVolumeResolution.Y +
							PageCoordToSplat.Y * PageTableVolumeResolution.X +
							PageCoordToSplat.X
						] = DstTileIndex;

					// Write tile data
					const FIntVector3 ParentVolumeCoordBase = (PageTableVolumeAABBMin + PageCoordToSplat) * SPARSE_VOLUME_TILE_RES * 2;
					for (int32 AttributesIdx = 0; AttributesIdx < 2; ++AttributesIdx)
					{
						if (Header.AttributesFormats[AttributesIdx] == PF_Unknown)
						{
							continue;
						}

						for (int32 Z = 0; Z < SPARSE_VOLUME_TILE_RES; ++Z)
						{
							for (int32 Y = 0; Y < SPARSE_VOLUME_TILE_RES; ++Y)
							{
								for (int32 X = 0; X < SPARSE_VOLUME_TILE_RES; ++X)
								{
									FVector4f DownsampledValue = FVector4f(0.0f, 0.0f, 0.0f, 0.0f);
									for (int32 OffsetIdx = 0; OffsetIdx < 8; ++OffsetIdx)
									{
										const FIntVector3 Offset = FIntVector3(OffsetIdx, OffsetIdx >> 1, OffsetIdx >> 2) & 1;
										const FIntVector3 SourceCoord = ParentVolumeCoordBase + FIntVector3(X, Y, Z) * 2 + Offset;
										const FVector4f SampleValue = Load(SourceCoord, MipLevel - 1, AttributesIdx, AddressingInfo);
										DownsampledValue += SampleValue;
									}
									DownsampledValue /= 8.0f;

									const FIntVector3 WriteCoord = FIntVector3(X, Y, Z);
									WriteTileDataVoxel(DstTileIndex - 1, WriteCoord, MipLevel, AttributesIdx, DownsampledValue); // -1 because we don't store the null tile in memory.
								}
							}
						}
					}
				});
		}

		if (!GenerateBorderVoxels(AddressingInfo, MipLevel, LinearAllocatedPages))
		{
			return false;
		}
	}

	return true;
}

bool FTextureData::GenerateBorderVoxels(const FTextureDataAddressingInfo& AddressingInfo, int32 MipLevel, const TArray<FIntVector3>& PageCoords)
{
	const FIntVector3 PageTableVolumeResolution = ShiftRightAndMax(Header.PageTableVolumeResolution, MipLevel, 1);

	ParallelFor(PageCoords.Num(), [this, MipLevel, &PageCoords, &PageTableVolumeResolution, &AddressingInfo](int32 PageIndex)
		{
			const FIntVector3& PageCoord = PageCoords[PageIndex];
			const int32 PageTableIndex = (PageCoord.Z * PageTableVolumeResolution.X * PageTableVolumeResolution.Y) + (PageCoord.Y * PageTableVolumeResolution.X) + PageCoord.X;
			const int32 DstTileIndex = (int32)MipMaps[MipLevel].PageTable[PageTableIndex] - 1; // -1 because we don't actually store the null tile in memory
			const FIntVector3 PageTableOffset = (Header.PageTableVolumeAABBMin >> MipLevel);
			for (int32 Z = -SPARSE_VOLUME_TILE_BORDER; Z < (SPARSE_VOLUME_TILE_RES + SPARSE_VOLUME_TILE_BORDER); ++Z)
			{
				for (int32 Y = -SPARSE_VOLUME_TILE_BORDER; Y < (SPARSE_VOLUME_TILE_RES + SPARSE_VOLUME_TILE_BORDER); ++Y)
				{
					for (int32 X = -SPARSE_VOLUME_TILE_BORDER; X < (SPARSE_VOLUME_TILE_RES + SPARSE_VOLUME_TILE_BORDER); ++X)
					{
						if (!IsInBounds(FIntVector3(X, Y, Z), FIntVector3::ZeroValue, FIntVector3(SPARSE_VOLUME_TILE_RES)))
						{
							const FIntVector3 LocalCoord = FIntVector3(X, Y, Z);
							const FIntVector3 VolumeCoord = (PageTableOffset + PageCoord) * SPARSE_VOLUME_TILE_RES + LocalCoord;
							const FVector4f Border0 = Load(VolumeCoord, MipLevel, 0, AddressingInfo);
							const FVector4f Border1 = Load(VolumeCoord, MipLevel, 1, AddressingInfo);
							WriteTileDataVoxel(DstTileIndex, LocalCoord, MipLevel, 0, Border0);
							WriteTileDataVoxel(DstTileIndex, LocalCoord, MipLevel, 1, Border1);
						}
					}
				}
			}
		});

	return true;
}

bool FTextureData::BuildDerivedData(const FTextureDataAddressingInfo& AddressingInfo, int32 NumMipLevelsGlobal, bool bMoveMip0FromThis, FDerivedTextureData& OutDerivedData)
{
	FTextureData MippedTextureData{};
	MippedTextureData.Header = Header;
	MippedTextureData.Header.UpdatePageTableFromGlobalNumMipLevels(NumMipLevelsGlobal);
	if (!MippedTextureData.Header.Validate(true /*bPrintToLog*/))
	{
		return false;
	}
	MippedTextureData.MipMaps.SetNum(1);
	if (bMoveMip0FromThis)
	{
		MippedTextureData.MipMaps[0] = MoveTemp(MipMaps[0]);
		MipMaps[0] = {};
	}
	else
	{
		MippedTextureData.MipMaps[0] = MipMaps[0];
	}

	// Page table changed due to UpdatePageTableFromGlobalNumMipLevels(), so we need to recreate it based on the old one
	if (MippedTextureData.Header.PageTableVolumeAABBMin != Header.PageTableVolumeAABBMin
		|| MippedTextureData.Header.PageTableVolumeAABBMax != Header.PageTableVolumeAABBMax
		|| MippedTextureData.Header.PageTableVolumeResolution != Header.PageTableVolumeResolution)
	{
		TArray<uint32> PageTableNew;
		PageTableNew.SetNumZeroed(MippedTextureData.Header.PageTableVolumeResolution.X * MippedTextureData.Header.PageTableVolumeResolution.Y * MippedTextureData.Header.PageTableVolumeResolution.Z);

		// Iterate over new page table and copy entries from old one
		for (int32 Z = MippedTextureData.Header.PageTableVolumeAABBMin.Z; Z < MippedTextureData.Header.PageTableVolumeAABBMax.Z; ++Z)
		{
			for (int32 Y = MippedTextureData.Header.PageTableVolumeAABBMin.Y; Y < MippedTextureData.Header.PageTableVolumeAABBMax.Y; ++Y)
			{
				for (int32 X = MippedTextureData.Header.PageTableVolumeAABBMin.X; X < MippedTextureData.Header.PageTableVolumeAABBMax.X; ++X)
				{
					FIntVector3 GlobalPageTableCoord(X, Y, Z);
					if (IsInBounds(GlobalPageTableCoord, Header.PageTableVolumeAABBMin, Header.PageTableVolumeAABBMax))
					{
						const FIntVector3 LocalPageTableCoordOld = GlobalPageTableCoord - Header.PageTableVolumeAABBMin;
						const int32 PageTableIndexOld = (LocalPageTableCoordOld.Z * Header.PageTableVolumeResolution.Y * Header.PageTableVolumeResolution.X) 
							+ (LocalPageTableCoordOld.Y * Header.PageTableVolumeResolution.X) 
							+ LocalPageTableCoordOld.X;
						check(MippedTextureData.MipMaps[0].PageTable.IsValidIndex(PageTableIndexOld));

						const FIntVector3 LocalPageTableCoordNew = GlobalPageTableCoord - MippedTextureData.Header.PageTableVolumeAABBMin;
						const int32 PageTableIndexNew = (LocalPageTableCoordNew.Z * MippedTextureData.Header.PageTableVolumeResolution.Y * MippedTextureData.Header.PageTableVolumeResolution.X) 
							+ (LocalPageTableCoordNew.Y * MippedTextureData.Header.PageTableVolumeResolution.X) 
							+ LocalPageTableCoordNew.X;
						check(PageTableNew.IsValidIndex(PageTableIndexNew));

						PageTableNew[PageTableIndexNew] = MippedTextureData.MipMaps[0].PageTable[PageTableIndexOld];
					}
				}
			}
		}

		MippedTextureData.MipMaps[0].PageTable = MoveTemp(PageTableNew);
	}

	// Generate border voxels of mip0
	{
		// Collect the page table coordinates of all non-zero pages
		TArray<FIntVector3> PageCoords;
		for (int32 PageZ = 0; PageZ < MippedTextureData.Header.PageTableVolumeResolution.Z; ++PageZ)
		{
			for (int32 PageY = 0; PageY < MippedTextureData.Header.PageTableVolumeResolution.Y; ++PageY)
			{
				for (int32 PageX = 0; PageX < MippedTextureData.Header.PageTableVolumeResolution.X; ++PageX)
				{
					const int32 PageIndex = PageZ * (MippedTextureData.Header.PageTableVolumeResolution.Y * MippedTextureData.Header.PageTableVolumeResolution.X) + PageY * MippedTextureData.Header.PageTableVolumeResolution.X + PageX;
					const uint32 PageTableEntry = MippedTextureData.MipMaps[0].PageTable[PageIndex];
					if (PageTableEntry != 0)
					{
						PageCoords.Add(FIntVector3(PageX, PageY, PageZ));
					}
				}
			}
		}

		if (!MippedTextureData.GenerateBorderVoxels(AddressingInfo, 0, PageCoords))
		{
			return false;
		}
	}

	// Generate all remaining mips. Also generates border voxels.
	if (!MippedTextureData.GenerateMipMaps(AddressingInfo))
	{
		return false;
	}

	OutDerivedData.Reset();
	OutDerivedData.Build(MippedTextureData);

	return true;
}

void FDerivedTextureData::Reset()
{
	NumPhysicalTiles = 0;
	MipPageRanges.Reset();
	PageTableCoords.Reset();
	PageTableTileIndices.Reset();
	PageTableParentIndices.Reset();
	PhysicalTileDataA.Reset();
	PhysicalTileDataB.Reset();
}

void FDerivedTextureData::Build(const FTextureData& MippedTextureData)
{
	Header = MippedTextureData.Header;

	const int32 NumMipLevels = MippedTextureData.MipMaps.Num();
	const SIZE_T FormatSize[] = { GPixelFormats[Header.AttributesFormats[0]].BlockBytes, GPixelFormats[Header.AttributesFormats[1]].BlockBytes };
	const SIZE_T TileByteSize[] = { SVT::NumVoxelsPerPaddedTile * FormatSize[0], SVT::NumVoxelsPerPaddedTile * FormatSize[1] };

	// Compute number of total tiles and pages, as well as the number and offset of pages per mip level
	MipPageRanges.SetNum(NumMipLevels);
	uint32 SrcNumPhysicalTilesTotal = 0;
	uint32 NumPagesTotal = 0;
	for (int32 MipLevel = NumMipLevels - 1; MipLevel >= 0; --MipLevel)
	{
		SrcNumPhysicalTilesTotal += MippedTextureData.MipMaps[MipLevel].NumPhysicalTiles;
		
		uint32 NumPagesMip = 0;
		for (uint32 Entry : MippedTextureData.MipMaps[MipLevel].PageTable)
		{
			NumPagesMip += (Entry != 0) ? 1 : 0;
		}
		MipPageRanges[MipLevel].PageOffset = NumPagesTotal;
		MipPageRanges[MipLevel].PageCount = NumPagesMip;
		NumPagesTotal += NumPagesMip;
	}

	// Reserve memory
	PageTableCoords.Reserve(NumPagesTotal);
	PageTableTileIndices.Reserve(NumPagesTotal);
	PageTableParentIndices.Reserve(NumPagesTotal);
	PhysicalTileDataA.SetNumUninitialized(SrcNumPhysicalTilesTotal * TileByteSize[0]);
	PhysicalTileDataB.SetNumUninitialized(SrcNumPhysicalTilesTotal * TileByteSize[1]);
	
	// Maps to help with deduplication and finding the parent index of a given page
	TMap<uint64, uint32> PhysicalTileDataRemap;
	TArray<TMap<uint32, uint32>> PageCoordToPageIndex;
	PageCoordToPageIndex.SetNum(NumMipLevels);

	uint32 NumWrittenTiles = 0;
	uint8* DstPhysicalTileDataA = PhysicalTileDataA.GetData();
	uint8* DstPhysicalTileDataB = PhysicalTileDataB.GetData();

	for (int32 MipLevel = NumMipLevels - 1; MipLevel >= 0; --MipLevel)
	{
		const uint8* SrcPhysicalTileDataA = MippedTextureData.MipMaps[MipLevel].PhysicalTileDataA.GetData();
		const uint8* SrcPhysicalTileDataB = MippedTextureData.MipMaps[MipLevel].PhysicalTileDataB.GetData();

		// Iterate over dense page table of this mip
		const FIntVector3 MipPageTableResolution = ShiftRightAndMax(MippedTextureData.Header.PageTableVolumeResolution, MipLevel, 1);
		for (int32 PageZ = 0; PageZ < MipPageTableResolution.Z; ++PageZ)
		{
			for (int32 PageY = 0; PageY < MipPageTableResolution.Y; ++PageY)
			{
				for (int32 PageX = 0; PageX < MipPageTableResolution.X; ++PageX)
				{
					const int32 LinearPageTableIndex = (PageZ * MipPageTableResolution.Y * MipPageTableResolution.X) + (PageY * MipPageTableResolution.X) + PageX;
					const uint32 Entry = MippedTextureData.MipMaps[MipLevel].PageTable[LinearPageTableIndex];

					if (Entry)
					{
						const SIZE_T SrcTileIdx = Entry - 1; // 0 encodes the null tile, but we don't store that tile explicitely
						
						// Hash the tile voxel data
						uint64 Hash = 0;
						if (FormatSize[0])
						{
							Hash = CityHash64((const char*)SrcPhysicalTileDataA + SrcTileIdx * TileByteSize[0], (uint32)TileByteSize[0]);
						}
						if (FormatSize[1])
						{
							Hash = CityHash64WithSeed((const char*)SrcPhysicalTileDataB + SrcTileIdx * TileByteSize[1], (uint32)TileByteSize[1], Hash);
						}

						// Have we processed an identical tile before?
						uint32 RemappedTileIndex = INDEX_NONE;
						if (uint32* RemappedTileIndexPtr = PhysicalTileDataRemap.Find(Hash))
						{
							// If so, we can simply reuse that tile
							RemappedTileIndex = *RemappedTileIndexPtr;
						}
						else
						{
							// Tile is unique, so copy it from the source data to the new compacted array
							RemappedTileIndex = NumWrittenTiles++;
							PhysicalTileDataRemap.Add(Hash, RemappedTileIndex);

							if (FormatSize[0])
							{
								FMemory::Memcpy(DstPhysicalTileDataA + RemappedTileIndex * TileByteSize[0], SrcPhysicalTileDataA +SrcTileIdx * TileByteSize[0], TileByteSize[0]);
							}
							if (FormatSize[1])
							{
								FMemory::Memcpy(DstPhysicalTileDataB + RemappedTileIndex * TileByteSize[1], SrcPhysicalTileDataB + SrcTileIdx * TileByteSize[1], TileByteSize[1]);
							}
						}

						// Insert page into hash map so child pages can look up its index
						const uint32 PackedPageCoord = SVT::PackX11Y11Z10(FIntVector3(PageX, PageY, PageZ));
						PageCoordToPageIndex[MipLevel].Add(PackedPageCoord, PageTableCoords.Num());
						
						PageTableCoords.Add(PackedPageCoord);
						PageTableTileIndices.Add(RemappedTileIndex);
						if (PageCoordToPageIndex.IsValidIndex(MipLevel + 1))
						{
							// Look up parent index based on mip level and parent page coord
							const uint32 ParentPageCoord = SVT::PackX11Y11Z10(FIntVector3(PageX / 2, PageY / 2, PageZ / 2));
							const uint32 ParentIndex = PageCoordToPageIndex[MipLevel + 1][ParentPageCoord];
							PageTableParentIndices.Add(ParentIndex);
						}
						else
						{
							// Root mip has no parent
							PageTableParentIndices.Add(INDEX_NONE);
						}
					}
				}
			}
		}
	}

	// Resize the arrays but don't bother shrinking; the gains are probably not massive and this object is expected to have a fairly short lifetime anyways.
	PhysicalTileDataA.SetNum(static_cast<int64>(NumWrittenTiles * TileByteSize[0]), EAllowShrinking::No);
	PhysicalTileDataB.SetNum(static_cast<int64>(NumWrittenTiles * TileByteSize[1]), EAllowShrinking::No);
	NumPhysicalTiles = NumWrittenTiles;

	// Validate
	{
#if DO_CHECK
		uint32 TotalPageCount = 0;
		for (int32 MipLevel = NumMipLevels - 1; MipLevel >= 0; --MipLevel)
		{
			check(MipPageRanges[MipLevel].PageOffset == TotalPageCount);
			const int32 NumPageTableEntries = MipPageRanges[MipLevel].PageCount;
			TotalPageCount += NumPageTableEntries;
			const bool bIsHighestMipLevel = MipLevel == (NumMipLevels - 1);
			check(!bIsHighestMipLevel || NumPageTableEntries <= 1); // Highest mip has a maximum of 1 page
			const FIntVector3 PageTableRes = ShiftRightAndMax(Header.PageTableVolumeResolution, MipLevel, 1);

			for (int32 i = 0; i < NumPageTableEntries; ++i)
			{
				const int32 PageIndex = MipPageRanges[MipLevel].PageOffset + i;
				const int32 PageX = (PageTableCoords[PageIndex] >> 0u) & 0x7FFu;
				const int32 PageY = (PageTableCoords[PageIndex] >> 11u) & 0x7FFu;
				const int32 PageZ = (PageTableCoords[PageIndex] >> 22u) & 0x3FFu;
				const int32 LinearPageIndex = (PageZ * (PageTableRes.X * PageTableRes.Y)) + (PageY * PageTableRes.X) + PageX;
				check(MippedTextureData.MipMaps[MipLevel].PageTable.IsValidIndex(LinearPageIndex));
				const uint32 PageTableEntry = MippedTextureData.MipMaps[MipLevel].PageTable[LinearPageIndex];
				check(PageTableEntry != 0 && PageTableEntry != INDEX_NONE); // Unpacked page coord must map to a valid entry in the dense page table
			}
		}
		check(TotalPageCount == PageTableCoords.Num());
#endif // DO_CHECK
	}
}

}
}

////////////////////////////////////////////////////////////////////////////////////////////////
