// Copyright Epic Games, Inc. All Rights Reserved.

#include "SparseVolumeTextureStreamingInstance.h"
#include "SparseVolumeTexture/ISparseVolumeTextureStreamingManager.h"
#include "HAL/IConsoleManager.h"

static float GSVTStreamingInstanceWindowSize = 5.0f;
static FAutoConsoleVariableRef CVarSVTStreamingInstanceWindowSize(
	TEXT("r.SparseVolumeTexture.Streaming.InstanceWindowSize"),
	GSVTStreamingInstanceWindowSize,
	TEXT("Window size in SVT frames around which requests to a SVT are considered to belong to the same instance. A streaming instance is an internal book keeping object to track playback of a SVT asset in a given context. Default: 5.0"),
	ECVF_RenderThreadSafe
);

static int32 GSVTStreamingForceEstimateFrameRate = 0;
static FAutoConsoleVariableRef CVarSVTStreamingForceEstimateFrameRate(
	TEXT("r.SparseVolumeTexture.Streaming.ForceEstimateFrameRate"),
	GSVTStreamingForceEstimateFrameRate,
	TEXT("Forces the streaming system to always estimate the SVT playback frame rate instead of using the explicit frame rate that is passed in with the streaming requests. Intended for debugging."),
	ECVF_RenderThreadSafe
);

static int32 GSVTStreamingRequestSizeGranularity = 128 * 1024;
static FAutoConsoleVariableRef CVarSVTStreamingRequestSizeGranularity(
	TEXT("r.SparseVolumeTexture.Streaming.RequestSizeGranularity"),
	GSVTStreamingRequestSizeGranularity,
	TEXT("Rounds up calculated streaming request sizes to a multiple of this value (in bytes). This avoids performance issues where the system might issue multiple very small requests."),
	ECVF_RenderThreadSafe
);

namespace UE
{
namespace SVT
{

bool FStreamingInstanceRequest::IsBlocking() const
{
	return EnumHasAnyFlags(Flags, EStreamingRequestFlags::Blocking);
}

bool FStreamingInstanceRequest::HasValidFrameRate() const
{
	return EnumHasAnyFlags(Flags, EStreamingRequestFlags::HasFrameRate);
}

FStreamingInstance::FStreamingInstance(uint32 InKey, int32 InNumFrames, const TArrayView<uint32>& InMipLevelStreamingSizes, const FStreamingInstanceRequest& Request)
	: MipLevelStreamingSizes(InMipLevelStreamingSizes)
	, Key(InKey)
	, NumFrames(InNumFrames)
{
	PreviousAverageRequestIssueTime = Request.Time;
	AverageRequestIssueTime = Request.Time;
	DeltaTime = 0.0;
	UpdateIndex = Request.UpdateIndex;
	PreviousAverageFrame = Request.FrameIndex;
	AverageFrame = Request.FrameIndex;
	NumRequestsThisUpdate = 1;
	LowestRequestedMip = Request.IsBlocking() ? FLT_MAX : Request.MipLevel;
	LowestRequestedBlockingMip = Request.IsBlocking() ? Request.MipLevel : FLT_MAX;
	EstimatedFrameRate = Request.HasValidFrameRate() ? Request.FrameRate : 0.0f;
	bPlayForwards = true; // No prior data, so just take a guess that playback is forwards
	bPlayBackwards = false;
	bIsBlocking = Request.IsBlocking();
}

void FStreamingInstance::AddRequest(const FStreamingInstanceRequest& Request)
{
	if (Request.UpdateIndex > UpdateIndex)
	{
		PreviousAverageRequestIssueTime = AverageRequestIssueTime;
		UpdateIndex = Request.UpdateIndex;
		PreviousAverageFrame = AverageFrame;
		NumRequestsThisUpdate = 0;
		LowestRequestedMip = FLT_MAX;
		LowestRequestedBlockingMip = FLT_MAX;
		bPlayForwards = false;
		bPlayBackwards = false;
		bIsBlocking = false;
	}

	// Calculate the average requested frame this update, taking into account potential wrapping when using looping playback. If this is the first request this update, just use its FrameIndex directly.
	AverageFrame = NumRequestsThisUpdate == 0 ? Request.FrameIndex : GetWrappedWeightedAverage(Request.FrameIndex, 1.0f, AverageFrame, NumRequestsThisUpdate, NumFrames);
	// Compute the new average request issue time based on the newly added request, the current average time and the number of already averaged requests.
	AverageRequestIssueTime = (AverageRequestIssueTime * NumRequestsThisUpdate + Request.Time) / (NumRequestsThisUpdate + 1.0f);
	// Estimate the the DeltaTime (and later down the frame rate) based on the average times requests were issued in the previous frame/update and the current one.
	DeltaTime = FMath::Max(0.0, AverageRequestIssueTime - PreviousAverageRequestIssueTime);
	EstimatedFrameRate = 0.0f;
	if (Request.HasValidFrameRate() && GSVTStreamingForceEstimateFrameRate == 0)
	{
		// Use the explicitly passed in frame rate if possible.
		EstimatedFrameRate = Request.FrameRate;
	}
	else if (DeltaTime > 0.0)
	{
		// Otherwise we estimate it based on the difference in requested frames and delta time.
		EstimatedFrameRate = FMath::Abs(GetShortestWrappedDistance(AverageFrame, PreviousAverageFrame, NumFrames)) / DeltaTime;
	}
	if (Request.IsBlocking())
	{
		LowestRequestedBlockingMip = FMath::Min(LowestRequestedBlockingMip, Request.MipLevel);
	}
	else
	{
		LowestRequestedMip = FMath::Min(LowestRequestedMip, Request.MipLevel);
	}
	const float DistanceFromPreviousFrame = GetShortestWrappedDistance(Request.FrameIndex, PreviousAverageFrame, NumFrames);
	bPlayForwards = bPlayForwards || (DistanceFromPreviousFrame > 0.0f);
	bPlayBackwards = bPlayBackwards || (DistanceFromPreviousFrame < 0.0f);
	bIsBlocking = bIsBlocking || Request.IsBlocking();

	++NumRequestsThisUpdate;
}

void FStreamingInstance::ComputeLowestMipLevelInBandwidthBudget(int64 BandwidthBudget)
{
	if (BandwidthBudget < 0)
	{
		// No bandwidth limit
		LowestMipInBandwidthBudget = 0.0f;
	}
	else
	{
		LowestMipInBandwidthBudget = MipLevelStreamingSizes.Num() - 1;

		// Walk from highest to lowest mip and find the fractional mip that can be streamed continuously without exceeding the budget. Could probably use binary search here in the future. 
		int64 PrevMipStreamingBandwidth = 0;
		for (int32 MipLevelIndex = MipLevelStreamingSizes.Num() - 1; MipLevelIndex >= 0; --MipLevelIndex)
		{
			const int64 MipStreamingSize = MipLevelStreamingSizes[MipLevelIndex];
			// Technically this assumes 1FPS for instances that aren't playing back at all (FrameRate==0.0), but this should be fine because we may still
			// need to stream in data for the requested frame and it is probably better to err on the safe side of things.
			const int64 MipStreamingBandwidth = FMath::CeilToInt64(MipStreamingSize * FMath::Max(1.0f, GetEstimatedFrameRate()));
			if (MipStreamingBandwidth > BandwidthBudget)
			{
				check(MipStreamingBandwidth >= PrevMipStreamingBandwidth);
				check(BandwidthBudget >= PrevMipStreamingBandwidth);
				const int64 NumBytesFromPrevMip = MipStreamingBandwidth - PrevMipStreamingBandwidth;
				const int64 NumBytesToLimit = BandwidthBudget - PrevMipStreamingBandwidth;
				const float MipFraction = static_cast<float>(NumBytesToLimit / static_cast<double>(NumBytesFromPrevMip));
				LowestMipInBandwidthBudget = (MipLevelIndex + 1) - MipFraction;
				break;
			}
			LowestMipInBandwidthBudget = MipLevelIndex;
			PrevMipStreamingBandwidth = MipStreamingBandwidth;
		}
	}
}

int64 FStreamingInstance::GetRequestedBandwidth(bool bZeroIfBlocking) const
{
	if (bZeroIfBlocking && bIsBlocking)
	{
		return 0;
	}
	const int64 StreamingSize = GetStreamingSize(LowestRequestedMip);
	const int64 RequestedBandwidthRaw = FMath::CeilToInt64(StreamingSize * FMath::Max(1.0f, GetEstimatedFrameRate()));
	const int64 RequestedBandwidth = ApplyDiscretization(RequestedBandwidthRaw);
	return RequestedBandwidth;
}

bool FStreamingInstance::IsFrameInWindow(float FrameIndex) const
{
	if (FrameIndex < 0.0f || FrameIndex >= static_cast<float>(NumFrames))
	{
		return false;
	}
	const float Distance = GetShortestWrappedDistance(AverageFrame, FrameIndex, NumFrames);
	const float WindowSize = FMath::Max(UE_SMALL_NUMBER, GSVTStreamingInstanceWindowSize);
	return FMath::Abs(Distance) <= WindowSize;
}

float FStreamingInstance::GetPrefetchMipLevel(float RequestedMipLevel, float Percentage) const
{
	const int64 RequestedMipLevelStreamingSize = GetStreamingSize(RequestedMipLevel);
	const int64 PrefetchStreamingSizeRaw = FMath::CeilToInt64(RequestedMipLevelStreamingSize * FMath::Clamp(Percentage, 0.0f, 1.0f));
	const int64 PrefetchStreamingSize = ApplyDiscretization(PrefetchStreamingSizeRaw);

	// Walk from the highest to lowest mip level and find the fractional mip level which corresponds to StreamingMemorySizeOf(RequestedMipLevel) * Percentage.
	int64 PrevMipStreamingSize = 0;
	for (int32 MipLevelIndex = MipLevelStreamingSizes.Num() - 1; MipLevelIndex >= 0; --MipLevelIndex)
	{
		const int64 MipStreamingSize = MipLevelStreamingSizes[MipLevelIndex];
		if (MipStreamingSize > PrefetchStreamingSize)
		{
			check(MipStreamingSize >= PrevMipStreamingSize);
			check(PrefetchStreamingSize >= PrevMipStreamingSize);
			const int64 NumBytesFromPrevMip = MipStreamingSize - PrevMipStreamingSize;
			const int64 NumBytesToLimit = PrefetchStreamingSize - PrevMipStreamingSize;
			const float MipFraction = static_cast<float>(NumBytesToLimit / static_cast<double>(NumBytesFromPrevMip));
			return (MipLevelIndex + 1) - MipFraction;
		}
		PrevMipStreamingSize = MipStreamingSize;
	}
	return 0.0f;
}

float FStreamingInstance::GetShortestWrappedDistance(float To, float From, int32 NumFrames)
{
	const float NumFramesF = static_cast<float>(NumFrames);
	check(NumFramesF >= 0.0f);
	check(From >= 0.0f && From < NumFramesF);
	check(To >= 0.0f && To < NumFramesF);
	const float Distance = To - From;
	// The wrapped distance must be the other way around, so we simply do NumFrames - NonWrappedDistance. In order to preserve
	// the sign, we need to operate on absolute values and add it back later.
	const float WrappedDistance = FMath::Max(0.0f, (NumFramesF - FMath::Abs(Distance))) * FMath::Sign(-Distance);
	const float Result = FMath::Abs(Distance) < FMath::Abs(WrappedDistance) ? Distance : WrappedDistance;
	return Result;
}

float FStreamingInstance::GetWrappedWeightedAverage(float ValueA, float WeightA, float ValueB, float WeightB, int32 NumFrames)
{
	const float NumFramesF = static_cast<float>(NumFrames);
	const float ShortestWrappedDistance = GetShortestWrappedDistance(ValueA, ValueB, NumFrames);
	const bool bIsWrapped = FMath::Abs(ShortestWrappedDistance) < FMath::Abs(ValueA - ValueB);

	float ValueAUnwrapped = ValueA;
	float ValueBUnwrapped = ValueB;
	if (bIsWrapped && ShortestWrappedDistance > 0)
	{
		ValueAUnwrapped += NumFramesF; // Forward wrapping: From > To (ValueB > ValueA)
	}
	else if (bIsWrapped && ShortestWrappedDistance < 0)
	{
		ValueBUnwrapped += NumFramesF; // Reverse wrapping: From < To (ValueB < ValueA)
	}

	float Result = (ValueAUnwrapped * WeightA + ValueBUnwrapped * WeightB) / FMath::Max(WeightA + WeightB, UE_SMALL_NUMBER);
	if (Result >= NumFramesF)
	{
		Result -= NumFramesF;
	}

	return Result;
}

int64 FStreamingInstance::ApplyDiscretization(int64 Value)
{
	const int64 DiscretizationStep = FMath::Max(GSVTStreamingRequestSizeGranularity, 1);
	Value = FMath::DivideAndRoundUp(Value, DiscretizationStep) * DiscretizationStep;
	return Value;
}

int64 FStreamingInstance::GetStreamingSize(float MipLevel) const
{
	const int32 LowerMipLevelIndex = FMath::Clamp(FMath::FloorToInt32(MipLevel), 0, MipLevelStreamingSizes.Num() - 1);
	const int32 UpperMipLevelIndex = FMath::Clamp(FMath::CeilToInt32(MipLevel), 0, MipLevelStreamingSizes.Num() - 1);
	check(MipLevelStreamingSizes.IsValidIndex(LowerMipLevelIndex));
	check(MipLevelStreamingSizes.IsValidIndex(UpperMipLevelIndex));
	const double InterpolatedStreamingSize = FMath::Lerp((double)MipLevelStreamingSizes[LowerMipLevelIndex], (double)MipLevelStreamingSizes[UpperMipLevelIndex], FMath::Frac(MipLevel));
	return static_cast<int64>(InterpolatedStreamingSize);
}

}
}
