// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "CoreMinimal.h"

namespace UE
{
namespace SVT
{

enum class EStreamingRequestFlags : uint8;

// A streaming request that can be passed to FStreamingInstance.
struct FStreamingInstanceRequest
{
	uint32 UpdateIndex = 0;
	double Time = 0.0;
	float FrameRate = 0.0f;
	float FrameIndex = 0.0f;
	float MipLevel = 0.0f;
	EStreamingRequestFlags Flags = {};

	explicit FStreamingInstanceRequest() = default;
	explicit FStreamingInstanceRequest(uint32 InUpdateIndex, double InTime, float InFrameRate, float InFrameIndex, float InMipLevel, EStreamingRequestFlags InFlags)
		: UpdateIndex(InUpdateIndex), Time(InTime), FrameRate(InFrameRate), FrameIndex(InFrameIndex), MipLevel(InMipLevel), Flags(InFlags) {}

	bool IsBlocking() const;
	bool HasValidFrameRate() const;
};

// Represents a streaming instance of a SVT. It is a "window" into the frame sequence that moves along the playback direction and is also identified by a "key", which is a user assigned value. 
// The FStreamingInstance is used to cache the prefetch direction, to compute the total bandwidth requirements of all streaming SVTs and to determine the allocated bandwidth for a given SVT instance.
class FStreamingInstance
{
public:
	explicit FStreamingInstance(uint32 Key, int32 NumFrames, const TArrayView<uint32>& MipLevelStreamingSizes, const FStreamingInstanceRequest& Request);
	void AddRequest(const FStreamingInstanceRequest& Request);
	// Returns the lowest mip level which can be continuously streamed while staying within the given bandwidth budget. 
	void ComputeLowestMipLevelInBandwidthBudget(int64 BandwidthBudget);
	// Returns the bandwidth required to stream the SVT at the requested mip level. This is relevant for non-blocking streaming, so it can optionally return zero if the instance is using blocking requests.
	int64 GetRequestedBandwidth(bool bZeroIfBlocking) const;
	// Returns true if the given frame is within the "window" of this instance.
	bool IsFrameInWindow(float FrameIndex) const;
	// Computes a fractional mip level which corresponds to StreamingMemorySizeOf(RequestedMipLevel) * Percentage.
	float GetPrefetchMipLevel(float RequestedMipLevel, float Percentage) const;
	uint32 GetKey() const { return Key; }
	uint32 GetUpdateIndex() const { return UpdateIndex; }
	float GetEstimatedFrameRate() const { return EstimatedFrameRate; }
	float GetAverageFrame() const { return AverageFrame; }
	float GetLowestRequestedMipLevel() const { return LowestRequestedMip; }
	float GetLowestMipLevelInBandwidthBudget() const { return LowestMipInBandwidthBudget; }
	bool IsPlayingForwards() const { return bPlayForwards; }
	bool IsPlayingBackwards() const { return bPlayBackwards; }

private:
	const TArrayView<uint32> MipLevelStreamingSizes;
	const uint32 Key;
	const int32 NumFrames;
	double PreviousAverageRequestIssueTime = 0.0;
	double AverageRequestIssueTime = 0.0;
	double DeltaTime = 0.0;
	uint32 UpdateIndex = 0;
	float PreviousAverageFrame = -1.0f;
	float AverageFrame = -1.0f; // Frame index this window is centered around
	float EstimatedFrameRate = 0.0f;
	int32 NumRequestsThisUpdate = 0;
	float LowestRequestedMip = FLT_MAX;
	float LowestRequestedBlockingMip = FLT_MAX;
	float LowestMipInBandwidthBudget = 0.0f;
	bool bPlayForwards = false;
	bool bPlayBackwards = false;
	bool bIsBlocking = false;

	// Returns the shortest signed distance between From and To, potentially wrapping around the 0 - NumFrames range.
	static float GetShortestWrappedDistance(float To, float From, int32 NumFrames);
	// Computes a weighted average between ValueA and ValueB, applying wrapping logic to both the calculation and the result.
	static float GetWrappedWeightedAverage(float ValueA, float WeightA, float ValueB, float WeightB, int32 NumFrames);
	// Rounds up to the next multiple of r.SparseVolumeTexture.Streaming.RequestSizeGranularity
	static int64 ApplyDiscretization(int64 Value);
	// Get the streaming size of the fractional mip level. GetStreamingSize(MipLevel) * FrameRate is the required peak bandwidth in bytes/s.
	int64 GetStreamingSize(float MipLevel) const;
};

}
}
