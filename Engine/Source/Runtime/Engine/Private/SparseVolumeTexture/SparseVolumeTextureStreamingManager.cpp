// Copyright Epic Games, Inc. All Rights Reserved.

#include "SparseVolumeTexture/SparseVolumeTextureStreamingManager.h"
#include "SparseVolumeTexture/SparseVolumeTextureUtility.h"
#include "HAL/IConsoleManager.h"
#include "SparseVolumeTexture/SparseVolumeTexture.h"
#include "RenderCore.h"
#include "RenderGraph.h"
#include "GlobalShader.h"
#include "ShaderCompilerCore.h" // AllowGlobalShaderLoad()
#include "Async/ParallelFor.h"
#include "SparseVolumeTextureTileDataTexture.h"
#include "SparseVolumeTextureUpload.h"
#include "SparseVolumeTextureStreamingInstance.h"
#include "GlobalRenderResources.h"
#include "ProfilingDebugging/IoStoreTrace.h"
#include "ProfilingDebugging/RealtimeGPUProfiler.h"
#include "RenderUtils.h"

#if WITH_EDITORONLY_DATA
#include "DerivedDataCache.h"
#include "DerivedDataRequestOwner.h"
#endif

DEFINE_LOG_CATEGORY(LogSparseVolumeTextureStreamingManager);

DECLARE_GPU_STAT(SVTStreaming);

DECLARE_MEMORY_STAT(TEXT("Total CPU Memory"), STAT_SVTCPUMemoryTotal, STATGROUP_SparseVolumeTextureMemory);
DECLARE_MEMORY_STAT_POOL(TEXT("PageTable GPU Memory"), STAT_SVTPageTableMemory, STATGROUP_SparseVolumeTextureMemory, FPlatformMemory::MCR_GPU);
DEFINE_STAT(STAT_SVTGPUMemoryTotal);

static int32 GSVTEnabled = 1;
static FAutoConsoleVariableRef CVarSVTEnabled(
	TEXT("r.SparseVolumeTexture"),
	GSVTEnabled,
	TEXT("Whether SparseVolumeTextures are supported at all. Toggles support for streaming and rendering SVTs."),
	ECVF_RenderThreadSafe | ECVF_ReadOnly
);

static int32 GSVTStreamingForceBlockingRequests = 0;
static FAutoConsoleVariableRef CVarSVTStreamingForceBlockingRequests(
	TEXT("r.SparseVolumeTexture.Streaming.ForceBlockingRequests"),
	GSVTStreamingForceBlockingRequests,
	TEXT("If enabled, all SVT streaming requests will block on completion, guaranteeing that requested mip levels are available in the same frame they have been requested in (if there is enough memory available to stream them in)."),
	ECVF_RenderThreadSafe
);

static int32 GSVTStreamingAsyncThread = 1;
static FAutoConsoleVariableRef CVarSVTStreamingAsync(
	TEXT("r.SparseVolumeTexture.Streaming.AsyncThread"),
	GSVTStreamingAsyncThread,
	TEXT("Perform most of the SVT streaming on an asynchronous worker thread instead of the rendering thread."),
	ECVF_RenderThreadSafe
);

static int32 GSVTStreamingEmptyPhysicalTileTextures = 0;
static FAutoConsoleVariableRef CVarSVTStreamingEmptyPhysicalTileTextures(
	TEXT("r.SparseVolumeTexture.Streaming.EmptyPhysicalTileTextures"),
	GSVTStreamingEmptyPhysicalTileTextures,
	TEXT("Streams out all streamable tiles of all physical tile textures."),
	ECVF_RenderThreadSafe
);

static int32 GSVTStreamingPrintMemoryStats = 0;
static FAutoConsoleVariableRef CVarSVTStreamingPrintMemoryStats(
	TEXT("r.SparseVolumeTexture.Streaming.PrintMemoryStats"),
	GSVTStreamingPrintMemoryStats,
	TEXT("Prints memory sizes of all frames of all SVTs registered with the streaming system."),
	ECVF_RenderThreadSafe
);

static int32 GSVTStreamingLogVerbosity = 1;
static FAutoConsoleVariableRef CVarSVTStreamingLogVerbosity(
	TEXT("r.SparseVolumeTexture.Streaming.LogVerbosity"),
	GSVTStreamingLogVerbosity,
	TEXT("0: no logging, 1: basic logging, 2: additional logging (might spam the log) 3: log everything (will spam the log)"),
	ECVF_RenderThreadSafe
);

static int32 GSVTStreamingMaxPendingRequests = 256 * 32;
static FAutoConsoleVariableRef CVarSVTStreamingMaxPendingRequests(
	TEXT("r.SparseVolumeTexture.Streaming.MaxPendingRequests"),
	GSVTStreamingMaxPendingRequests,
	TEXT("Maximum number of IO requests that can be pending for installation."),
	ECVF_RenderThreadSafe | ECVF_ReadOnly
);

static int32 GSVTStreamingRequestSize = -1;
static FAutoConsoleVariableRef CVarSVTStreamingRequestSize(
	TEXT("r.SparseVolumeTexture.Streaming.RequestSize"),
	GSVTStreamingRequestSize,
	TEXT("IO request size in KiB. The SVT streaming manager will attempt to create IO requests of roughly this size. Default: -1 (unlimited)"),
	ECVF_RenderThreadSafe
);

static int32 GSVTStreamingBandwidthLimit = 512;
static FAutoConsoleVariableRef CVarSVTStreamingBandwidthLimit(
	TEXT("r.SparseVolumeTexture.Streaming.BandwidthLimit"),
	GSVTStreamingBandwidthLimit,
	TEXT("Bandwidth limit for SVT streaming in MiB/s. When requests exceed this limit, the system will stream at lower mip levels instead."),
	ECVF_RenderThreadSafe
);

static int32 GSVTStreamingInstanceCleanupThreshold = 5;
static FAutoConsoleVariableRef CVarSVTStreamingInstanceCleanupThreshold(
	TEXT("r.SparseVolumeTexture.Streaming.InstanceCleanupThreshold"),
	GSVTStreamingInstanceCleanupThreshold,
	TEXT("Number of SVT streaming system updates to wait until unused streaming instances are cleaned up. A streaming instance is an internal book keeping object to track playback of a SVT asset in a given context. Default: 5"),
	ECVF_RenderThreadSafe
);

// When in editor, data is streamed from DDC. The request finishes in a callback on some other thread. This callback currently reads FPendingRequest::SVTHandle and FPendingRequest::RequestVersion
// and writes to FPendingRequest::SharedBuffer and FPendingRequest::State. Other than this, FPendingRequest is never accessed from multiple threads at the same time. However, in order to avoid race conditions, 
// all accesses to FPendingRequest should be guarded by using the following macro. Contention only happens if the streaming manager is currently accessing a request during an update and a DDC request 
// finishes at the exact same time this access happens. While this is possible, it is very unlikely, so the performance impact of always using the lock should be minimal.
#if WITH_EDITORONLY_DATA
#define LOCK_PENDING_REQUEST(PendingRequest) FScopeLock Lock(&PendingRequest.DDCAsyncGuard)
#else
#define LOCK_PENDING_REQUEST(PendingRequest)
#endif

namespace UE
{
namespace SVT
{

TGlobalResource<FStreamingManager> GStreamingManager;

IStreamingManager& GetStreamingManager()
{
	return GStreamingManager;
}

struct FStreamingUpdateParameters
{
	FStreamingManager* StreamingManager = nullptr;
};

class FStreamingUpdateTask
{
public:
	explicit FStreamingUpdateTask(const FStreamingUpdateParameters& InParams) : Parameters(InParams) {}

	FStreamingUpdateParameters Parameters;

	void DoTask(ENamedThreads::Type CurrentThread, const FGraphEventRef& MyCompletionGraphEvent)
	{
		Parameters.StreamingManager->InstallReadyRequests();
	}

	static ESubsequentsMode::Type	GetSubsequentsMode() { return ESubsequentsMode::TrackSubsequents; }
	ENamedThreads::Type				GetDesiredThread() { return ENamedThreads::AnyNormalThreadNormalTask; }
	FORCEINLINE TStatId				GetStatId() const { return TStatId(); }
};

FStreamingManager::FStreamingManager() = default;
// needed in module to forward declare some members
FStreamingManager::~FStreamingManager() = default;


void FStreamingManager::InitRHI(FRHICommandListBase& RHICmdList)
{
	using namespace UE::DerivedData;

	if (!UseSparseVolumeTextures(GMaxRHIShaderPlatform))
	{
		return;
	}

	MaxPendingRequests = GSVTStreamingMaxPendingRequests;
	PendingRequests.SetNum(MaxPendingRequests);
	PageTableUpdater = MakeUnique<FPageTableUpdater>();

#if WITH_EDITORONLY_DATA
	RequestOwner = MakeUnique<FRequestOwner>(EPriority::Highest);
	RequestOwnerBlocking = MakeUnique<FRequestOwner>(EPriority::Blocking);
#endif
}

void FStreamingManager::ReleaseRHI()
{
	if (!UseSparseVolumeTextures(GMaxRHIShaderPlatform))
	{
		return;
	}
}

void FStreamingManager::Add_GameThread(UStreamableSparseVolumeTexture* SparseVolumeTexture)
{
	if (!UseSparseVolumeTextures(GMaxRHIShaderPlatform) || !SparseVolumeTexture)
	{
		return;
	}

	FNewSparseVolumeTextureInfo NewSVTInfo{};
	const int32 NumFrames = SparseVolumeTexture->GetNumFrames();
	NewSVTInfo.SVT = SparseVolumeTexture;
	NewSVTInfo.SVTName = SparseVolumeTexture->GetFName();
	NewSVTInfo.FormatA = SparseVolumeTexture->GetFormat(0);
	NewSVTInfo.FormatB = SparseVolumeTexture->GetFormat(1);
	NewSVTInfo.FallbackValueA = SparseVolumeTexture->GetFallbackValue(0);
	NewSVTInfo.FallbackValueB = SparseVolumeTexture->GetFallbackValue(1);
	NewSVTInfo.NumMipLevelsGlobal = SparseVolumeTexture->GetNumMipLevels();
	NewSVTInfo.StreamingPoolSizeFactor = SparseVolumeTexture->StreamingPoolSizeFactor;
	NewSVTInfo.NumPrefetchFrames = SparseVolumeTexture->NumberOfPrefetchFrames;
	NewSVTInfo.PrefetchPercentageStepSize = SparseVolumeTexture->PrefetchPercentageStepSize;
	NewSVTInfo.PrefetchPercentageBias = SparseVolumeTexture->PrefetchPercentageBias;
	NewSVTInfo.FrameInfo.SetNum(NumFrames);

	for (int32 FrameIdx = 0; FrameIdx < NumFrames; ++FrameIdx)
	{
		USparseVolumeTextureFrame* SVTFrame = SparseVolumeTexture->GetFrame(FrameIdx);
		FFrameInfo& FrameInfo = NewSVTInfo.FrameInfo[FrameIdx];
		FrameInfo.Resources = SVTFrame->GetResources();
		FrameInfo.TextureRenderResources = SVTFrame->TextureRenderResources;
		check(FrameInfo.TextureRenderResources);
	}


	ENQUEUE_RENDER_COMMAND(SVTAdd)(
		[this, NewSVTInfoCaptured = MoveTemp(NewSVTInfo), SVTName = SparseVolumeTexture->GetName()](FRHICommandListImmediate& RHICmdList) mutable /* Required to be able to move from NewSVTInfoCaptured inside the lambda */
		{
			// We need to fully initialize the SVT streaming state (including resource creation) to ensure that valid resources exist before FillUniformBuffers() is called.
			// This is why we can't defer resource creation until BeginAsyncUpdate() is called.
			FRDGBuilder GraphBuilder(RHICmdList, RDG_EVENT_NAME("SVT::FStreamingManager::Add(%s)", *SVTName));
			AddInternal(GraphBuilder, MoveTemp(NewSVTInfoCaptured));
			GraphBuilder.Execute();
		});
}

void FStreamingManager::Remove_GameThread(UStreamableSparseVolumeTexture* SparseVolumeTexture)
{
	if (!UseSparseVolumeTextures(GMaxRHIShaderPlatform) || !SparseVolumeTexture)
	{
		return;
	}
	ENQUEUE_RENDER_COMMAND(SVTRemove)(
		[this, SparseVolumeTexture](FRHICommandListImmediate& RHICmdList)
		{
			RemoveInternal(SparseVolumeTexture);
		});
}

void FStreamingManager::Request_GameThread(UStreamableSparseVolumeTexture* SparseVolumeTexture, uint32 StreamingInstanceKey, float FrameRate, float FrameIndex, float MipLevel, EStreamingRequestFlags Flags)
{
	if (!UseSparseVolumeTextures(GMaxRHIShaderPlatform) || !SparseVolumeTexture)
	{
		return;
	}
	ENQUEUE_RENDER_COMMAND(SVTRequest)(
		[this, SparseVolumeTexture, StreamingInstanceKey, FrameRate, FrameIndex, MipLevel, Flags](FRHICommandListImmediate& RHICmdList)
		{
			Request(SparseVolumeTexture, StreamingInstanceKey, FrameRate, FrameIndex, MipLevel, Flags);
		});
}

void FStreamingManager::Update_GameThread()
{
	if (!UseSparseVolumeTextures(GMaxRHIShaderPlatform))
	{
		return;
	}
	ENQUEUE_RENDER_COMMAND(SVTUpdate)(
		[](FRHICommandListImmediate& RHICmdList)
		{
			FRDGBuilder GraphBuilder(RHICmdList);
			const bool bUseAsyncThread = false; // No need to spin up a thread if we immediately wait on it anyways.
			GStreamingManager.BeginAsyncUpdate(GraphBuilder, bUseAsyncThread);
			GStreamingManager.EndAsyncUpdate(GraphBuilder);
			GraphBuilder.Execute();
		});
}

void FStreamingManager::Request(UStreamableSparseVolumeTexture* SparseVolumeTexture, uint32 StreamingInstanceKey, float FrameRate, float FrameIndex, float MipLevel, EStreamingRequestFlags Flags)
{
	check(IsInRenderingThread());
	if (!UseSparseVolumeTextures(GMaxRHIShaderPlatform) || !SparseVolumeTexture)
	{
		return;
	}

	FStreamingInfo* SVTInfo = FindStreamingInfo(SparseVolumeTexture);
	if (SVTInfo)
	{
		const int32 NumFrames = SVTInfo->PerFrameInfo.Num();
		const int32 FrameIndexI32 = FMath::FloorToInt32(FrameIndex);
		if (FrameIndexI32 < 0 || FrameIndexI32 >= NumFrames)
		{
			return;
		}

		const bool bBlocking = !!(Flags & EStreamingRequestFlags::Blocking);
		const double CurrentTime = (FTimespan(FDateTime::Now().GetTicks()) - FTimespan(InitTime.GetTicks())).GetTotalSeconds();

		// Try to find a FStreamingInstance around the requested frame index and add the current request to it. This will inform us about which direction we need to prefetch into.
		const FStreamingInstanceRequest StreamingInstanceRequest(UpdateIndex, CurrentTime, FrameRate, FrameIndex, MipLevel, Flags);
		FStreamingInstance* StreamingInstance = SVTInfo->GetAndUpdateStreamingInstance(StreamingInstanceKey, StreamingInstanceRequest);
		check(StreamingInstance);

		// Make sure the number of prefetched frames doesn't exceed the total number of frames.
		// Not only does this make no sense, it also breaks the wrap around logic in the loop below if there is reverse playback.
		const int32 NumPrefetchFrames = FMath::Clamp(SVTInfo->NumPrefetchFrames, 0, NumFrames);

		// No prefetching for blocking requests. Making the prefetches blocking would increase latency even more and making them non-blocking could lead
		// to situations where lower mips are already streamed in in subsequent frames but can't be used because dependent higher mips haven't finished streaming
		// due to non-blocking requests.
		// SVT_TODO: This can still break if blocking and non-blocking requests of the same frames/mips are made to the same SVT. We would need to cancel already scheduled non-blocking requests and reissue them as blocking.
		// Or alternatively we could just block on all requests if we detect this case. If we had a single DDC request owner per request, we could just selectively wait on already scheduled non-blocking requests.
		const int32 OffsetMagnitude = !bBlocking ? NumPrefetchFrames : 0;
		const int32 LowerFrameOffset = StreamingInstance->IsPlayingBackwards() ? -OffsetMagnitude : 0;
		const int32 UpperFrameOffset = StreamingInstance->IsPlayingForwards() ? OffsetMagnitude : 0;
		const float PrefetchPercentageStepSize = FMath::Clamp(SVTInfo->PrefetchPercentageStepSize / 100.0f, 0.0f, 1.0f);
		const float PrefetchPercentageBias = FMath::Clamp(SVTInfo->PrefetchPercentageBias / 100.0f, -1.0f, 1.0f);

		for (int32 i = LowerFrameOffset; i <= UpperFrameOffset; ++i)
		{
			// Wrap around on both positive and negative numbers, assuming (i + NumFrames) >= 0. See the comment on NumPrefetchFrames.
			const int32 RequestFrameIndex = (FrameIndexI32 + i + NumFrames) % NumFrames;
			const float MipPercentage = FMath::Clamp((1.0f - FMath::Abs(i) * PrefetchPercentageStepSize) + (i != 0 ? PrefetchPercentageBias : 0.0f), 0.0f, 1.0f);
			const float RequestMipLevel = StreamingInstance->GetPrefetchMipLevel(MipLevel, MipPercentage);
			const int32 RequestMipLevelI = FMath::FloorToInt32(RequestMipLevel);
			const float LowestMipLevelFraction = 1.0f - FMath::Frac(RequestMipLevel);
			const uint8 Priority = bBlocking ? UINT8_MAX : FMath::Max(0, OffsetMagnitude - FMath::Abs(i));
			FStreamingRequest Request;
			Request.Key.SVTHandle = SVTInfo->SVTHandle;
			Request.Key.FrameIndex = RequestFrameIndex;
			Request.Payload.StreamingInstance = StreamingInstance;
			Request.Payload.MipLevelMask = ~0u << RequestMipLevelI;
			Request.Payload.LowestMipFraction = LowestMipLevelFraction;
			for (int32 j = RequestMipLevelI; j < Request.Payload.Priorities.Num(); ++j)
			{
				Request.Payload.Priorities[j] = Priority;
			}
			AddRequest(Request);
		}

		ActiveStreamingInstances.Add(StreamingInstance);
	}
}

void FStreamingManager::BeginAsyncUpdate(FRDGBuilder& GraphBuilder, bool bUseAsyncThread)
{
	check(IsInRenderingThread());
	check(!AsyncState.bUpdateActive);
	if (!UseSparseVolumeTextures(GMaxRHIShaderPlatform) || StreamingInfo.IsEmpty())
	{
		return;
	}

	RDG_EVENT_SCOPE_STAT(GraphBuilder, SVTStreaming, "SVT::StreamingBeginAsyncUpdate");
	RDG_CSV_STAT_EXCLUSIVE_SCOPE(GraphBuilder, SVTStreaming);
	SCOPED_NAMED_EVENT_TEXT("SVT::StreamingBeginAsyncUpdate", FColor::Green);

	if (GSVTStreamingLogVerbosity > 2)
	{
		UE_LOGF(LogSparseVolumeTextureStreamingManager, Display, "SVT Streaming Update %i", UpdateIndex);
	}

	AsyncState = {};
	AsyncState.bUpdateActive = true;

	// Clean up unused streaming instances
	for (TUniquePtr<FStreamingInfo>& SVTInfo : StreamingInfo)
	{
		if (SVTInfo)
		{
			SVTInfo->StreamingInstances.RemoveAll([&](const TUniquePtr<FStreamingInstance>& Instance) { return (UpdateIndex - Instance->GetUpdateIndex()) > (uint32)FMath::Max(1, GSVTStreamingInstanceCleanupThreshold); });
		}
	}

	// For debugging, we can stream out ALL tiles
	if (GSVTStreamingEmptyPhysicalTileTextures != 0)
	{
		for (TUniquePtr<FStreamingInfo>& SVTInfo : StreamingInfo)
		{
			if (!SVTInfo)
			{
				continue;
			}
			const int32 NumFrames = SVTInfo->PerFrameInfo.Num();
			for (int32 FrameIndex = 0; FrameIndex < NumFrames; ++FrameIndex)
			{
				FFrameInfo& FrameInfo = SVTInfo->PerFrameInfo[FrameIndex];
				const bool bHasRootTile = FrameInfo.Resources->StreamingMetaData.HasRootTile();
				for (TConstSetBitIterator It(FrameInfo.StreamingTiles, bHasRootTile ? 1 : 0); It; ++It)
				{
					const uint32 TileIndex = It.GetIndex();
					FrameInfo.StreamingTiles[TileIndex] = false;
					FrameInfo.ResidentTiles[TileIndex] = false;
					check(FrameInfo.TileAllocations[TileIndex] != 0);
					SVTInfo->TileAllocator.Free(FrameInfo.TileAllocations[TileIndex]);
					FrameInfo.TileAllocations[TileIndex] = 0;
				}
				FrameInfo.TileIndexToPendingRequestIndex.Reset();
				InvalidatedSVTFrames.Add(&FrameInfo);
			}
		}

		// Cancel all IO requests
		for (FPendingRequest& PendingRequest : PendingRequests)
		{
			LOCK_PENDING_REQUEST(PendingRequest);
			PendingRequest.Reset();
		}
		NumPendingRequests = 0;

		GSVTStreamingEmptyPhysicalTileTextures = 0;
	}

	if (GSVTStreamingPrintMemoryStats != 0)
	{
		for (TUniquePtr<FStreamingInfo>& SVTInfo : StreamingInfo)
		{
			if (!SVTInfo)
			{
				continue;
			}

			double MinFrameMiB = FLT_MAX;
			double MaxFrameMiB = -FLT_MAX;
			double SumFrameMiB = 0.0;

			const int32 NumFrames = SVTInfo->PerFrameInfo.Num();
			UE_LOGF(LogSparseVolumeTextureStreamingManager, Display, "Memory stats for SVT '%ls':", *SVTInfo->SVTName.ToString());
		
			for (int32 FrameIdx = 0; FrameIdx < NumFrames; ++FrameIdx)
			{
				const FFrameInfo& FrameInfo = SVTInfo->PerFrameInfo[FrameIdx];
				const uint32 Size = FrameInfo.Resources->StreamingMetaData.TileDataOffsets.Last() - FrameInfo.Resources->StreamingMetaData.GetRootTileSize();
				const uint32 NumTiles = FrameInfo.Resources->StreamingMetaData.GetNumStreamingTiles();

				MinFrameMiB = FMath::Min(MinFrameMiB, Size / 1024.0 / 1024.0);
				MaxFrameMiB = FMath::Max(MaxFrameMiB, Size / 1024.0 / 1024.0);
				SumFrameMiB += Size / 1024.0 / 1024.0;

				UE_LOGF(LogSparseVolumeTextureStreamingManager, Display, "SVT Frame %3u: Size: %5.4f MiB Num Tiles: %u", FrameIdx, Size / 1024.0f / 1024.0f, NumTiles);
			}

			UE_LOGF(LogSparseVolumeTextureStreamingManager, Display, "SVT Frame Stats: Min: %3.2f MiB, Max: %3.2f MiB, Avg: %3.2f, Total All: %3.2f MiB", (float)MinFrameMiB, (float)MaxFrameMiB, (float)(SumFrameMiB / NumFrames), SumFrameMiB);
		}

		GSVTStreamingPrintMemoryStats = 0;
	}

	ComputeBandwidthLimit();
	FilterRequests();
	IssueRequests();
	AsyncState.NumReadyRequests = DetermineReadyRequests();

	// Do a first pass over all the tiles to be uploaded to compute the upload buffer size requirements.
	TileDataTexturesToUpdate.Reset();
	{
		const int32 StartPendingRequestIndex = (NextPendingRequestIndex + MaxPendingRequests - NumPendingRequests) % MaxPendingRequests;
		for (int32 i = 0; i < AsyncState.NumReadyRequests; ++i)
		{
			const int32 PendingRequestIndex = (StartPendingRequestIndex + i) % MaxPendingRequests;
			FPendingRequest& PendingRequest = PendingRequests[PendingRequestIndex];
			LOCK_PENDING_REQUEST(PendingRequest);

			// Skip if request was cancelled
			if (!PendingRequest.IsValid())
			{
				continue;
			}

			FStreamingInfo* SVTInfo = FindStreamingInfo(PendingRequest.SVTHandle);
			check(SVTInfo);

			// Prepare tile data texture for upload
			const FTileDataTexture::EUploaderState UploaderState = SVTInfo->TileDataTexture->GetUploaderState();
			check(UploaderState == FTileDataTexture::EUploaderState::Ready || UploaderState == FTileDataTexture::EUploaderState::Reserving);
			if (UploaderState == FTileDataTexture::EUploaderState::Ready)
			{
				SVTInfo->TileDataTexture->BeginReserveUpload();
			}

			const int32 FormatSizeA = GPixelFormats[SVTInfo->FormatA].BlockBytes;
			const int32 FormatSizeB = GPixelFormats[SVTInfo->FormatB].BlockBytes;
			FFrameInfo& FrameInfo = SVTInfo->PerFrameInfo[PendingRequest.FrameIndex];

			uint32 NumVoxelsA = 0;
			uint32 NumVoxelsB = 0;
			FrameInfo.Resources->StreamingMetaData.GetNumVoxelsInTileRange(PendingRequest.TileOffset, PendingRequest.TileCount, FormatSizeA, FormatSizeB, &FrameInfo.StreamingTiles, NumVoxelsA, NumVoxelsB);

			SVTInfo->TileDataTexture->ReserveUpload(PendingRequest.TileCount, NumVoxelsA, NumVoxelsB);

			TileDataTexturesToUpdate.Add(SVTInfo->TileDataTexture.Get());
		}

		for (FTileDataTexture* TileDataTexture : TileDataTexturesToUpdate)
		{
			TileDataTexture->EndReserveUpload();
			TileDataTexture->BeginUpload(GraphBuilder);
		}
	}

	// Start async processing
	FStreamingUpdateParameters Parameters;
	Parameters.StreamingManager = this;
	
	check(AsyncTaskEvents.IsEmpty());
	if (GSVTStreamingAsyncThread && bUseAsyncThread)
	{
		AsyncState.bUpdateIsAsync = true;
		AsyncTaskEvents.Add(TGraphTask<FStreamingUpdateTask>::CreateTask().ConstructAndDispatchWhenReady(Parameters));
	}
	else
	{
		InstallReadyRequests();
	}
}

void FStreamingManager::EndAsyncUpdate(FRDGBuilder& GraphBuilder)
{
	check(IsInRenderingThread());
	if (!UseSparseVolumeTextures(GMaxRHIShaderPlatform) || StreamingInfo.IsEmpty())
	{
		return;
	}
	check(AsyncState.bUpdateActive);

	RDG_EVENT_SCOPE_STAT(GraphBuilder, SVTStreaming, "SVT::StreamingEndAsyncUpdate");
	RDG_CSV_STAT_EXCLUSIVE_SCOPE(GraphBuilder, SVTStreaming);
	SCOPED_NAMED_EVENT_TEXT("SVT::StreamingEndAsyncUpdate", FColor::Green);

	// Wait for async processing to finish
	if (AsyncState.bUpdateIsAsync)
	{
		check(!AsyncTaskEvents.IsEmpty());
		FTaskGraphInterface::Get().WaitUntilTasksComplete(AsyncTaskEvents, ENamedThreads::GetRenderThread_Local());
	}
	AsyncTaskEvents.Empty();

	// Issue the actual data uploads
	for (FTileDataTexture* TileDataTexture : TileDataTexturesToUpdate)
	{
		TileDataTexture->EndUpload(GraphBuilder);
	}

	// Update page table with newly streamed in/out pages and make sure descendant pages in the hierarchy have correct fallback values
	PatchPageTable(GraphBuilder);

	check(AsyncState.NumReadyRequests <= NumPendingRequests);
	NumPendingRequests -= AsyncState.NumReadyRequests;
	++UpdateIndex;
	AsyncState.bUpdateActive = false;
	AsyncState.bUpdateIsAsync = false;

	SET_MEMORY_STAT(STAT_SVTCPUMemoryTotal, GetAllocatedSize());
}

const FStreamingDebugInfo* FStreamingManager::GetStreamingDebugInfo(FRDGBuilder& GraphBuilder) const
{
	const int64 BandwidthLimit = FMath::Max(GSVTStreamingBandwidthLimit, 1) * 1024LL * 1024LL;
	const bool bLimitApplies = (GSVTStreamingBandwidthLimit > 0 && (TotalRequestedBandwidth > BandwidthLimit));
	const double DownscaleFactor = bLimitApplies ? ((double)BandwidthLimit / (double)TotalRequestedBandwidth) : 1.0;
	const int32 NumSVTs = StreamingInfo.Num();

	FStreamingDebugInfo::FSVT* DebugInfoSVTs = GraphBuilder.AllocPODArray<FStreamingDebugInfo::FSVT>(NumSVTs);
	int32 SVTArrayWriteSlot = 0;
	for (const TUniquePtr<FStreamingInfo>& SVTInfo : StreamingInfo)
	{
		if (!SVTInfo)
		{
			continue;
		}
		const int32 NumFrames = SVTInfo->PerFrameInfo.Num();
		const int32 NumInstances = SVTInfo->StreamingInstances.Num();
		
		float* ResidencyPercentages = GraphBuilder.AllocPODArray<float>(NumFrames);
		float* StreamingPercentages = GraphBuilder.AllocPODArray<float>(NumFrames);
		for (int32 FrameIndex = 0; FrameIndex < NumFrames; ++FrameIndex)
		{
			const FFrameInfo& FrameInfo = SVTInfo->PerFrameInfo[FrameIndex];
			ResidencyPercentages[FrameIndex] = FrameInfo.ResidentTiles.CountSetBits() / (float)FMath::Max(FrameInfo.ResidentTiles.Num(), 1);
			StreamingPercentages[FrameIndex] = FrameInfo.StreamingTiles.CountSetBits() / (float)FMath::Max(FrameInfo.StreamingTiles.Num(), 1);
		}
		
		FStreamingDebugInfo::FSVT::FInstance* Instances = GraphBuilder.AllocPODArray<FStreamingDebugInfo::FSVT::FInstance>(NumInstances);
		for (int32 InstanceIndex = 0; InstanceIndex < NumInstances; ++InstanceIndex)
		{
			FStreamingInstance* StreamingInstance = SVTInfo->StreamingInstances[InstanceIndex].Get();
			if (StreamingInstance)
			{
				FStreamingDebugInfo::FSVT::FInstance Instance;
				Instance.Key = StreamingInstance->GetKey();
				Instance.Frame = StreamingInstance->GetAverageFrame();
				Instance.FrameRate = StreamingInstance->GetEstimatedFrameRate();
				Instance.RequestedBandwidth = float(StreamingInstance->GetRequestedBandwidth(true) / 1024.0 / 1024.0);
				Instance.AllocatedBandwidth = Instance.RequestedBandwidth * DownscaleFactor;
				Instance.RequestedMip = StreamingInstance->GetLowestRequestedMipLevel();
				Instance.InBudgetMip = StreamingInstance->GetLowestMipLevelInBandwidthBudget();

				Instances[InstanceIndex] = Instance;
			}
		}

		FStreamingDebugInfo::FSVT& SVT = DebugInfoSVTs[SVTArrayWriteSlot++];
		SVT.AssetName = SVTInfo->SVTName;
		SVT.FrameResidencyPercentages = ResidencyPercentages;
		SVT.FrameStreamingPercentages = StreamingPercentages;
		SVT.Instances = Instances;
		SVT.NumFrames = NumFrames;
		SVT.NumInstances = NumInstances;
	}

	FStreamingDebugInfo* DebugInfo = GraphBuilder.AllocPOD<FStreamingDebugInfo>();
	DebugInfo->SVTs = DebugInfoSVTs;
	DebugInfo->NumSVTs = SVTArrayWriteSlot;
	DebugInfo->RequestedBandwidth = float(TotalRequestedBandwidth / 1024.0 / 1024.0);
	DebugInfo->BandwidthLimit = float(BandwidthLimit / 1024.0 / 1024.0);
	DebugInfo->BandwidthScale = float(DownscaleFactor);
	
	return DebugInfo;
}

void FStreamingManager::AddInternal(FRDGBuilder& GraphBuilder, FNewSparseVolumeTextureInfo&& NewSVTInfo)
{
	check(IsInRenderingThread());
	check(!AsyncState.bUpdateActive);
	if (!ensure(!SparseVolumeTextureToHandle.Contains(NewSVTInfo.SVT)))
	{
		return;
	}

	RDG_EVENT_SCOPE_STAT(GraphBuilder, SVTStreaming, "SVT::StreamingAddInternal");
	RDG_CSV_STAT_EXCLUSIVE_SCOPE(GraphBuilder, SVTStreaming);
	SCOPED_NAMED_EVENT_TEXT("SVT::StreamingAddInternal", FColor::Green);

	// Allocate SVTHandle, which is actually simply an index into the sparse StreamingInfo array.
	int32 LowestFreeIndexSearchStart = 0;
	const uint16 SVTHandle = StreamingInfo.EmplaceAtLowestFreeIndex(LowestFreeIndexSearchStart, MakeUnique<FStreamingInfo>());
	SparseVolumeTextureToHandle.Add(NewSVTInfo.SVT, SVTHandle);

	const int32 NumFrames = NewSVTInfo.FrameInfo.Num();

	FStreamingInfo& SVTInfo = *StreamingInfo[SVTHandle];
	SVTInfo.SVTHandle = SVTHandle;
	SVTInfo.SVTName = NewSVTInfo.SVTName;
	SVTInfo.FormatA = NewSVTInfo.FormatA;
	SVTInfo.FormatB = NewSVTInfo.FormatB;
	SVTInfo.FallbackValueA = NewSVTInfo.FallbackValueA;
	SVTInfo.FallbackValueB = NewSVTInfo.FallbackValueB;
	SVTInfo.NumPrefetchFrames = NewSVTInfo.NumPrefetchFrames;
	SVTInfo.PrefetchPercentageStepSize = NewSVTInfo.PrefetchPercentageStepSize;
	SVTInfo.PrefetchPercentageBias = NewSVTInfo.PrefetchPercentageBias;
	SVTInfo.PerFrameInfo = MoveTemp(NewSVTInfo.FrameInfo);
	SVTInfo.MipLevelStreamingSize.SetNumZeroed(NewSVTInfo.NumMipLevelsGlobal);

	const int32 FormatSizes[] = { GPixelFormats[SVTInfo.FormatA].BlockBytes, GPixelFormats[SVTInfo.FormatB].BlockBytes };

	int32 NumRootPhysicalTiles = 0;
	int32 NumRootVoxelsA = 0;
	int32 NumRootVoxelsB = 0;
	int32 MaxNumPhysicalTiles = 0;
	for (int32 FrameIdx = 0; FrameIdx < NumFrames; ++FrameIdx)
	{
		FFrameInfo& FrameInfo = SVTInfo.PerFrameInfo[FrameIdx];
		check(FrameInfo.TextureRenderResources && FrameInfo.TextureRenderResources->IsInitialized());
		const FResources* Resources = FrameInfo.Resources;
		const FPageTopology& Topology = Resources->Topology;

		const int32 NumPhysicalTiles = Resources->StreamingMetaData.GetNumTiles();
		MaxNumPhysicalTiles = FMath::Max(NumPhysicalTiles, MaxNumPhysicalTiles);

		FrameInfo.NumMipLevels = Topology.MipInfo.Num();
		FrameInfo.TileAllocations.SetNumZeroed(NumPhysicalTiles);

		const int32 NumPagesTotal = Topology.NumPages();
		FrameInfo.ResidentPages.SetNum(NumPagesTotal, false);
		FrameInfo.InvalidatedPages.SetNum(NumPagesTotal, false);
		FrameInfo.ResidentTiles.SetNum(NumPhysicalTiles, false);
		FrameInfo.StreamingTiles.SetNum(NumPhysicalTiles, false);
		
		if (!Topology.MipInfo.IsEmpty() && Topology.MipInfo.Last().PageCount > 0)
		{
			check(Topology.MipInfo.Last().PageOffset == 0);
			check(Topology.MipInfo.Last().PageCount == 1);
			++NumRootPhysicalTiles;
			FTileInfo RootTileInfo = Resources->StreamingMetaData.GetTileInfo(0, FormatSizes[0], FormatSizes[1]);
			NumRootVoxelsA += RootTileInfo.NumVoxels[0];
			NumRootVoxelsB += RootTileInfo.NumVoxels[1];
		}

		// Determine the high water mark of per frame streaming size when streaming at any given mip level.
		// Find the maximum referenced tile index + 1, which is the index in TileDataOffsets (a prefix sum with N+1 elements) at which the cumulative size of this tile and all prior tiles is stored.
		uint32 MaxReferencedTileIndexPlusOne = 0;
		for (int32 MipLevel = FrameInfo.NumMipLevels - 2; MipLevel >= 0; --MipLevel)
		{
			if (Topology.MipInfo.IsValidIndex(MipLevel))
			{
				const FPageTopology::FMip& MipInfo = Topology.MipInfo[MipLevel];
				for (uint32 PageIndex = MipInfo.PageOffset; PageIndex < (MipInfo.PageOffset + MipInfo.PageCount); ++PageIndex)
				{
					const uint32 TileIndex = Topology.TileIndices[PageIndex];
					MaxReferencedTileIndexPlusOne = FMath::Max(MaxReferencedTileIndexPlusOne, TileIndex + 1);
				}
				const uint32 MipStreamingSize = Resources->StreamingMetaData.TileDataOffsets[MaxReferencedTileIndexPlusOne] - Resources->StreamingMetaData.GetRootTileSize();
				SVTInfo.MipLevelStreamingSize[MipLevel] = FMath::Max(SVTInfo.MipLevelStreamingSize[MipLevel], MipStreamingSize);
			}
		}
	}

	// Create RHI resources and upload root tile data
	{
		const float TileFactor = NumFrames <= 1 ? 1.0f : FMath::Clamp(NewSVTInfo.StreamingPoolSizeFactor, 1.0f, static_cast<float>(NumFrames));
		const int32 NumPhysicalTilesCapacity = FMath::Max(1, NumRootPhysicalTiles + FMath::CeilToInt32(TileFactor * MaxNumPhysicalTiles)); // Ensure a minimum size of 1
		const FIntVector3 TileDataVolumeResolutionInTiles = FTileDataTexture::GetVolumeResolutionInTiles(NumPhysicalTilesCapacity);

		SVTInfo.TileDataTexture = MakeUnique<FTileDataTexture>(TileDataVolumeResolutionInTiles, SVTInfo.FormatA, SVTInfo.FormatB, SVTInfo.FallbackValueA, SVTInfo.FallbackValueB);
		SVTInfo.TileDataTexture->InitResource(GraphBuilder.RHICmdList);
		SVTInfo.TileAllocator.Init(SVTInfo.TileDataTexture->GetResolutionInTiles());

		FTileUploader RootTileUploader;
		RootTileUploader.Init(GraphBuilder, NumRootPhysicalTiles + 1 /*null tile*/, NumRootVoxelsA, NumRootVoxelsB, SVTInfo.FormatA, SVTInfo.FormatB);

		// Create null tile
		{
			const uint32 NullTileCoord = 0; // Implicitly allocated when we created the tile allocator
			FTileUploader::FAddResult AddResult = RootTileUploader.Add_GetRef(1 /*NumTiles*/, 0 /*NumVoxelsA*/, 0 /*NumVoxelsB*/);
			FMemory::Memcpy(AddResult.PackedPhysicalTileCoordsPtr, &NullTileCoord, sizeof(NullTileCoord));
			if (SVTInfo.FormatA != PF_Unknown)
			{
				FMemory::Memzero(AddResult.OccupancyBitsPtrs[0], SVT::NumOccupancyWordsPerPaddedTile * sizeof(uint32));
				FMemory::Memzero(AddResult.TileDataOffsetsPtrs[0], sizeof(uint32));
			}
			if (SVTInfo.FormatB != PF_Unknown)
			{
				FMemory::Memzero(AddResult.OccupancyBitsPtrs[1], SVT::NumOccupancyWordsPerPaddedTile * sizeof(uint32));
				FMemory::Memzero(AddResult.TileDataOffsetsPtrs[1], sizeof(uint32));
			}
			// No need to write to TileDataPtrA and TileDataPtrB because we zeroed out all the occupancy bits.
		}

		const bool bAsyncCompute = UseAsyncComputeForStreaming();

		// Process frames
		for (int32 FrameIdx = 0; FrameIdx < NumFrames; ++FrameIdx)
		{
			FFrameInfo& FrameInfo = SVTInfo.PerFrameInfo[FrameIdx];
			const FResources* Resources = FrameInfo.Resources;
			const FPageTopology& Topology = Resources->Topology;
			const int32 NumMipLevels = Topology.MipInfo.Num();

			// Create page table
			{
				FIntVector3 PageTableResolution = Resources->Header.PageTableVolumeResolution;
				PageTableResolution = FIntVector3(FMath::Max(1, PageTableResolution.X), FMath::Max(1, PageTableResolution.Y), FMath::Max(1, PageTableResolution.Z));

				const EPixelFormat PageEntryFormat = PF_R32_UINT;
				const ETextureCreateFlags Flags = TexCreate_ShaderResource | TexCreate_UAV | TexCreate_3DTiling | TexCreate_ReduceMemoryWithTilingMode;
				FRDGTexture* PageTableRDG = GraphBuilder.CreateTexture(FRDGTextureDesc::Create3D(PageTableResolution, PageEntryFormat, FClearValueBinding::Black, Flags, (uint8)NumMipLevels), TEXT("SparseVolumeTexture.PageTableTexture"));

				// Clear page table to zero
				for (int32 MipLevelIndex = 0; MipLevelIndex < NumMipLevels; ++MipLevelIndex)
				{
					FRDGTextureUAV* UAV = GraphBuilder.CreateUAV(FRDGTextureUAVDesc(PageTableRDG, MipLevelIndex, PageEntryFormat));
					AddClearUAVPass(GraphBuilder, UAV, FUintVector4(ForceInitToZero), bAsyncCompute ? ERDGPassFlags::AsyncCompute : ERDGPassFlags::Compute);
				}

				FrameInfo.PageTableTexture = GraphBuilder.ConvertToExternalTexture(PageTableRDG);
				GraphBuilder.UseExternalAccessMode(PageTableRDG, ERHIAccess::SRVMask, ERHIPipeline::All);
#if STATS
				const int64 MemorySize = static_cast<int64>(RHICalcTexturePlatformSize(FrameInfo.PageTableTexture->GetRHI()->GetDesc()).Size);
				INC_MEMORY_STAT_BY(STAT_SVTPageTableMemory, MemorySize);
				INC_MEMORY_STAT_BY(STAT_SVTGPUMemoryTotal, MemorySize);
#endif
			}

			// Initialize TextureRenderResources
			GraphBuilder.RHICmdList.UpdateTextureReference(FrameInfo.TextureRenderResources->PageTableTextureReferenceRHI, FrameInfo.PageTableTexture->GetRHI());
			GraphBuilder.RHICmdList.UpdateTextureReference(FrameInfo.TextureRenderResources->PhysicalTileDataATextureReferenceRHI, SVTInfo.TileDataTexture->GetTileDataTextureA() ? SVTInfo.TileDataTexture->GetTileDataTextureA()->GetRHI() : GBlackVolumeTexture->TextureRHI.GetReference());
			GraphBuilder.RHICmdList.UpdateTextureReference(FrameInfo.TextureRenderResources->PhysicalTileDataBTextureReferenceRHI, SVTInfo.TileDataTexture->GetTileDataTextureB() ? SVTInfo.TileDataTexture->GetTileDataTextureB()->GetRHI() : GBlackVolumeTexture->TextureRHI.GetReference());
			FrameInfo.TextureRenderResources->Header = Resources->Header;
			FrameInfo.TextureRenderResources->TileDataTextureResolution = SVTInfo.TileDataTexture->GetResolutionInTiles() * SPARSE_VOLUME_TILE_RES_PADDED;
			FrameInfo.TextureRenderResources->FrameIndex = FrameIdx;
			FrameInfo.TextureRenderResources->NumLogicalMipLevels = NumMipLevels;

			// Upload root mip data and update page tables
			if (!Topology.MipInfo.IsEmpty() && Topology.MipInfo.Last().PageCount > 0)
			{
				check(!Resources->RootData.IsEmpty());
				FTileAllocator::FAllocation PreviousAllocation;
				const uint32 TileCoord = SVTInfo.TileAllocator.Allocate(UpdateIndex, 0 /*FreeThreshold*/, FrameIdx, 0 /*TileIndexInFrame*/, -1 /*TilePriority*/, true, PreviousAllocation);
				check(TileCoord != INDEX_NONE);
				check(TileCoord != 0);
				check(!PreviousAllocation.bIsAllocated); // We should never have to evict another tile to upload a root tile
				FrameInfo.TileAllocations[0] = TileCoord;
				FrameInfo.ResidentTiles[0] = true;
				FrameInfo.StreamingTiles[0] = true; // This tile is not technically streaming, but StreamingTiles is a super set of ResidentTiles and represents tiles that are resident or about to be

				const FTileInfo RootTileInfo = Resources->StreamingMetaData.GetTileInfo(0, FormatSizes[0], FormatSizes[1]);
				const int32 NumVoxelsA = RootTileInfo.NumVoxels[0];
				const int32 NumVoxelsB = RootTileInfo.NumVoxels[1];
				FTileUploader::FAddResult AddResult = RootTileUploader.Add_GetRef(1, NumVoxelsA, NumVoxelsB);

				FMemory::Memcpy(AddResult.PackedPhysicalTileCoordsPtr, &TileCoord, sizeof(TileCoord));
				for (int32 AttributesIdx = 0; AttributesIdx < 2; ++AttributesIdx)
				{
					if (FormatSizes[AttributesIdx] > 0)
					{
						// Occupancy bits
						const uint8* SrcOccupancyBits = Resources->RootData.GetData() + RootTileInfo.OccupancyBitsOffsets[AttributesIdx];
						check(AddResult.OccupancyBitsPtrs[AttributesIdx]);
						FMemory::Memcpy(AddResult.OccupancyBitsPtrs[AttributesIdx], SrcOccupancyBits, RootTileInfo.OccupancyBitsSizes[AttributesIdx]);

						// Per-tile offsets into tile data
						check(AddResult.TileDataOffsetsPtrs[AttributesIdx]);
						reinterpret_cast<uint32*>(AddResult.TileDataOffsetsPtrs[AttributesIdx])[0] = AddResult.TileDataBaseOffsets[AttributesIdx] + 0; // + 0 because we're only uploading this single tile in the current batch

						// Tile data
						const uint8* SrcTileData = Resources->RootData.GetData() + RootTileInfo.VoxelDataOffsets[AttributesIdx];
						check(AddResult.TileDataPtrs[AttributesIdx]);
						FMemory::Memcpy(AddResult.TileDataPtrs[AttributesIdx], SrcTileData, RootTileInfo.VoxelDataSizes[AttributesIdx]);
					}
				}
			}

			InvalidatedSVTFrames.Add(&FrameInfo);
		}

		RootTileUploader.ResourceUploadTo(GraphBuilder, SVTInfo.TileDataTexture->GetTileDataTextureA(), SVTInfo.TileDataTexture->GetTileDataTextureB(), SVTInfo.FallbackValueA, SVTInfo.FallbackValueB);
	}

	// Add requests for all mips the first frame. This is necessary for cases where UAnimatedSparseVolumeTexture or UStaticSparseVolumeTexture
	// are directly bound to the material without getting a specific frame through USparseVolumeTextureFrame::GetFrameAndIssueStreamingRequest().
	const int32 NumMipLevelsFrame0 = SVTInfo.PerFrameInfo[0].NumMipLevels;
	FStreamingRequest Request;
	Request.Key.SVTHandle = SVTHandle;
	Request.Key.FrameIndex = 0;
	Request.Payload.MipLevelMask = UINT16_MAX;
	Request.Payload.LowestMipFraction = 1.0f;
	AddRequest(Request);
}

void FStreamingManager::RemoveInternal(UStreamableSparseVolumeTexture* SparseVolumeTexture)
{
	check(IsInRenderingThread());
	check(!AsyncState.bUpdateActive);
	FStreamingInfo* SVTInfo = FindStreamingInfo(SparseVolumeTexture);
	if (SVTInfo)
	{
		// Remove any requests for this SVT
		TArray<FFrameKey> RequestsToRemove;
		for (auto& Pair : RequestsHashTable)
		{
			if (Pair.Key.SVTHandle == SVTInfo->SVTHandle)
			{
				RequestsToRemove.Add(Pair.Key);
			}
		}
		for (const FFrameKey& Key : RequestsToRemove)
		{
			RequestsHashTable.Remove(Key);
		}

		// Cancel any pending requests
		for (FPendingRequest& PendingRequest : PendingRequests)
		{
			LOCK_PENDING_REQUEST(PendingRequest);
			if (PendingRequest.SVTHandle == SVTInfo->SVTHandle)
			{
				PendingRequest.Reset();
			}
		}

		// Make sure ActiveStreamingInstances doesn't have any references to this SVT anymore
		for (TUniquePtr<FStreamingInstance>& StreamingInstance : SVTInfo->StreamingInstances)
		{
			ActiveStreamingInstances.Remove(StreamingInstance.Get());
		}

		// Release resources
		for (FFrameInfo& FrameInfo : SVTInfo->PerFrameInfo)
		{
#if STATS
			const int64 MemorySize = static_cast<int64>(RHICalcTexturePlatformSize(FrameInfo.PageTableTexture->GetRHI()->GetDesc()).Size);
			DEC_MEMORY_STAT_BY(STAT_SVTPageTableMemory, MemorySize);
			DEC_MEMORY_STAT_BY(STAT_SVTGPUMemoryTotal, MemorySize);
#endif
			FrameInfo.PageTableTexture.SafeRelease();
			InvalidatedSVTFrames.Remove(&FrameInfo);
		}
		if (SVTInfo->TileDataTexture)
		{
			SVTInfo->TileDataTexture->ReleaseResource();
			SVTInfo->TileDataTexture.Reset();
		}

		StreamingInfo.RemoveAt(SVTInfo->SVTHandle, 1);
		SparseVolumeTextureToHandle.Remove(SparseVolumeTexture);
	}
}

void FStreamingManager::AddRequest(const FStreamingRequest& Request)
{
	FRequestPayload* ExistingRequestPayload = RequestsHashTable.Find(Request.Key);
	if (ExistingRequestPayload)
	{
		ExistingRequestPayload->MipLevelMask |= Request.Payload.MipLevelMask;
		const int32 ExistingLowestMip = FMath::CountTrailingZeros(ExistingRequestPayload->MipLevelMask);
		const int32 IncomingLowestMip = FMath::CountTrailingZeros(Request.Payload.MipLevelMask);
		if (IncomingLowestMip == ExistingLowestMip)
		{
			ExistingRequestPayload->LowestMipFraction = FMath::Max(ExistingRequestPayload->LowestMipFraction, Request.Payload.LowestMipFraction);
		}
		else if (IncomingLowestMip < ExistingLowestMip)
		{
			ExistingRequestPayload->LowestMipFraction = Request.Payload.LowestMipFraction;
		}
		for (int32 i = 0; i < Request.Payload.Priorities.Num(); ++i)
		{
			ExistingRequestPayload->Priorities[i] = FMath::Max(ExistingRequestPayload->Priorities[i], Request.Payload.Priorities[i]);
		}
	}
	else
	{
		RequestsHashTable.Add(Request.Key, Request.Payload);
	}
}

void FStreamingManager::ComputeBandwidthLimit()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(SVT::StreamingComputeBandwidthLimit);

	// Get the total requested bandwidth and initialize all streaming instances to not apply any limit.
	TotalRequestedBandwidth = 0;
	for (FStreamingInstance* StreamingInstance : ActiveStreamingInstances)
	{
		check(StreamingInstance->GetUpdateIndex() == UpdateIndex);
		TotalRequestedBandwidth += StreamingInstance->GetRequestedBandwidth(true /*bZeroIfBlocking*/);
		StreamingInstance->ComputeLowestMipLevelInBandwidthBudget(-1); // -1 sets the LowestMipLevelInBandwidthBuget to 0.0f
	}

	// Apply the limit if the cvar is set and the requested bandwidth exceeds the limit.
	const int64 AvailableBandwidth = FMath::Max(GSVTStreamingBandwidthLimit, 1) * 1024LL * 1024LL;
	if (GSVTStreamingBandwidthLimit > 0 && (TotalRequestedBandwidth > AvailableBandwidth))
	{
		// This is the factor by which we need to scale down the bandwidth of all currently active streaming instances.
		const double DownscaleFactor = ((double)AvailableBandwidth / (double)TotalRequestedBandwidth);
		if (GSVTStreamingLogVerbosity > 2)
		{
			UE_LOGF(LogSparseVolumeTextureStreamingManager, Display, "Total Requested Bandwidth: %f MiB/s, Total Available Bandwidth: %f MiB/s, Percentage: %f %%", 
				float(TotalRequestedBandwidth / 1024.0 / 1024.0), float(AvailableBandwidth / 1024.0 / 1024.0), float(DownscaleFactor * 100.0));
		}

		for (FStreamingInstance* StreamingInstance : ActiveStreamingInstances)
		{
			const int64 RequestedBandWidth = StreamingInstance->GetRequestedBandwidth(true /*bZeroIfBlocking*/);
			const int64 AllocatedBandwidth = FMath::CeilToInt64(RequestedBandWidth * DownscaleFactor);
			StreamingInstance->ComputeLowestMipLevelInBandwidthBudget(AllocatedBandwidth);

			if (GSVTStreamingLogVerbosity > 2)
			{
				UE_LOGF(LogSparseVolumeTextureStreamingManager, Display, "Key: %u, Requested: %f MiB, Allocated: %f MiB, FPS: %f, InBudgetMipLevel: %f",
					StreamingInstance->GetKey(), float(RequestedBandWidth / 1024.0 / 1024.0), float(AllocatedBandwidth / 1024.0 / 1024.0), StreamingInstance->GetEstimatedFrameRate(), StreamingInstance->GetLowestMipLevelInBandwidthBudget());
			}
		}
	}

	// We don't need this set after this point, so clear it for the next frames requests.
	ActiveStreamingInstances.Reset();
}

void FStreamingManager::FilterRequests()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(SVT::StreamingFilterRequests);

	TileRangesToStream.Reset();

	// Convert requests into contiguous ranges of tiles with the same priority
	for (const auto& RequestPair : RequestsHashTable)
	{
		const FFrameKey& RequestKey = RequestPair.Key;
		const FRequestPayload& RequestValue = RequestPair.Value;
		FStreamingInfo* SVTInfo = FindStreamingInfo(RequestKey.SVTHandle);
		if (!SVTInfo || !SVTInfo->PerFrameInfo.IsValidIndex(RequestKey.FrameIndex))
		{
			continue;
		}

		FFrameInfo& FrameInfo = SVTInfo->PerFrameInfo[RequestKey.FrameIndex];
		const FPageTopology& Topology = FrameInfo.Resources->Topology;
		const FTileStreamingMetaData& StreamingMetaData = FrameInfo.Resources->StreamingMetaData;
		const int32 FirstStreamingMipLevel = FrameInfo.NumMipLevels - 2;
		const int32 LowestRequestedMipI = static_cast<int32>(FMath::CountTrailingZeros((uint32)RequestValue.MipLevelMask));
		const float LowestRequestedMipF = LowestRequestedMipI + (1.0f - FMath::Clamp(RequestValue.LowestMipFraction, 0.0f, 1.0f));
		const float LowestAllowedMipF = FMath::Clamp(RequestValue.StreamingInstance ? RequestValue.StreamingInstance->GetLowestMipLevelInBandwidthBudget() : 0.0f, LowestRequestedMipF, FrameInfo.NumMipLevels - 1.0f); // Clamp to ensure Allows >= Requested
		const int32 LowestAllowedMipI = FMath::FloorToInt32(LowestAllowedMipF);
		const float LowestMipToStreamF = FMath::Max(LowestAllowedMipF, LowestRequestedMipF);
		const int32 LowestMipToStreamI = FMath::FloorToInt32(LowestMipToStreamF);

		// Keep track of all tiles requested in this frame and also of the tiles requested only within the current priority
		const uint32 NumTiles = FrameInfo.TileAllocations.Num();
		TBitArray<> RequestedTiles = FrameInfo.StreamingTiles;
		TBitArray<> RequestedTilesInCurrentPriority = TBitArray<>(false, NumTiles);

		// StreamingTiles must be a super set of ResidentTiles
		check(TBitArray<>::BitwiseAND(FrameInfo.StreamingTiles, FrameInfo.ResidentTiles, EBitwiseOperatorFlags::MaxSize) == FrameInfo.ResidentTiles);

		const uint32 TargetRequestSize = GSVTStreamingRequestSize <= 0 ? UINT32_MAX : (uint32)FMath::Clamp(GSVTStreamingRequestSize, 1, int32(UINT32_MAX / 1024)) * 1024u;

		// Creates a FTileRange for each contiguous range of set bits in RequestedTilesInCurrentPriority and appends it to TileRangesToStream
		auto MakeTileRanges = [&](uint8 Priority)
		{
			bool bResetCounters = true;
			uint32 TileOffset = 0;
			uint32 TileCount = 0;
			for (TConstSetBitIterator It(RequestedTilesInCurrentPriority); It; ++It)
			{
				const uint32 TileIndex = It.GetIndex();
				if (bResetCounters)
				{
					TileOffset = TileIndex;
					TileCount = 0;
					bResetCounters = false;
				}
				++TileCount;

				const bool bIsLastTile = !RequestedTilesInCurrentPriority.IsValidIndex(TileIndex + 1);
				const bool bIsNextTileBitSet = !bIsLastTile && RequestedTilesInCurrentPriority[TileIndex + 1];
				bool bReachedTargetRequestSize = false;
				if (bIsNextTileBitSet)
				{
					// End the current batch if adding the next tile would get us past the target request size.
					// TileDataOffsets has N+1 elements, so accessing with +2 is fine because we checked that +1 is < N.
					const uint32 RequestSizeIncludingNextTile = StreamingMetaData.TileDataOffsets[TileIndex + 2] - StreamingMetaData.TileDataOffsets[TileOffset];
					bReachedTargetRequestSize = RequestSizeIncludingNextTile >= TargetRequestSize;
				}

				// Is the next bit unset? If so, this tile is the end of the current range
				if (bIsLastTile || !bIsNextTileBitSet || bReachedTargetRequestSize)
				{
					FTileRange& TileRange = TileRangesToStream.AddDefaulted_GetRef();
					TileRange.SVTHandle = RequestKey.SVTHandle;
					TileRange.TileOffset = TileOffset;
					TileRange.TileCount = TileCount;
					TileRange.FrameIndex = RequestKey.FrameIndex;
					TileRange.Priority = Priority;

					bResetCounters = true;
				}
			}
		};

		uint32 PrevPriority = 0xFFFFFFFFu;
		for (int32 MipLevelIndex = FirstStreamingMipLevel; MipLevelIndex >= LowestRequestedMipI; --MipLevelIndex)
		{
			// Skip if mip level was not requested
			if (((uint32)RequestValue.MipLevelMask & (1u << MipLevelIndex)) == 0)
			{
				continue;
			}

			const uint8 Priority = RequestValue.Priorities[MipLevelIndex];
			const bool bIsBlocking = Priority == uint8(-1);

			// Skip if mip level is outside the allowed range for the bandwidth budget and is not a blocking request
			if (MipLevelIndex < LowestAllowedMipI && !bIsBlocking)
			{
				continue;
			}

			// When the priority changes, we need to create tile ranges based on the currently set bets in RequestedTilesInCurrentPriority.
			if (Priority != PrevPriority)
			{
				MakeTileRanges(PrevPriority);
				RequestedTilesInCurrentPriority.SetRange(0, NumTiles, false);
			}
			PrevPriority = Priority;

			auto GetFractionalPageCount = [](uint32 PageCount, float FractionalMipLevel)
			{
				const float Fraction = 1.0f - FMath::Frac(FractionalMipLevel);
				const uint32 ResultPageCount = FMath::CeilToInt32((float)PageCount * Fraction);
				check(ResultPageCount <= PageCount);
				return ResultPageCount;
			};

			// Get the range of pages to try and stream in
			const FPageTopology::FMip& MipInfo = Topology.MipInfo[MipLevelIndex];
			uint32 PageCount = MipInfo.PageCount;
			if (MipLevelIndex == LowestRequestedMipI)
			{
				PageCount = GetFractionalPageCount(MipInfo.PageCount, LowestRequestedMipF);
			}
			if (MipLevelIndex == LowestAllowedMipI && !bIsBlocking) // Apply the fractional allowed mip limit only on non-blocking requests
			{
				// If the lowest requested mip is the same integer mip as the lowest allowed mip, take the maximum fractional mip -> minimum bandwidth
				const float FractionalMip = MipLevelIndex == LowestRequestedMipI ? FMath::Max(LowestRequestedMipF, LowestAllowedMipF) : LowestAllowedMipF;
				PageCount = GetFractionalPageCount(MipInfo.PageCount, FractionalMip);
			}
			const uint32 PageBegin = MipInfo.PageOffset;
			const uint32 PageEnd = MipInfo.PageOffset + PageCount;

			// Mark requested tiles
			for (uint32 PageIndex = PageBegin; PageIndex < PageEnd; ++PageIndex)
			{
				const uint32 TileIndex = Topology.TileIndices[PageIndex];
				if (!RequestedTiles[TileIndex])
				{
					RequestedTiles[TileIndex] = true;
					RequestedTilesInCurrentPriority[TileIndex] = true;
				}

				// Tell the allocator that we're still interested in this tile
				if (FrameInfo.StreamingTiles[TileIndex])
				{
					check(FrameInfo.TileAllocations[TileIndex] != 0);
					const uint32 TilePriority = FrameInfo.ResidentTiles.Num() - TileIndex; // Higher value means higher priority, but we want higher tiles to be streamed out first
					SVTInfo->TileAllocator.UpdateUsage(UpdateIndex, FrameInfo.TileAllocations[TileIndex], TilePriority);
				}
			}
		}

		// After processing the lowest/last mip level, we need to create one final batch of tile ranges
		MakeTileRanges(PrevPriority);
	}

	RequestsHashTable.Reset();

	// Sort by priority
	auto PriorityPredicate = [](const auto& A, const auto& B) { return A.Priority != B.Priority ? (A.Priority > B.Priority) : A.TileOffset < B.TileOffset; };
	TileRangesToStream.Sort(PriorityPredicate);

	const int32 MaxSelectedRequests = MaxPendingRequests - NumPendingRequests;
	const int32 NumSelectedRequests = FMath::Min(MaxSelectedRequests, TileRangesToStream.Num());

	TileRangesToStream.SetNum(NumSelectedRequests, EAllowShrinking::No);
}

void FStreamingManager::IssueRequests()
{
	using namespace UE::DerivedData;

	if (TileRangesToStream.IsEmpty())
	{
		return;
	}

	TRACE_CPUPROFILER_EVENT_SCOPE(SVT::StreamingIssueRequests);

#if WITH_EDITORONLY_DATA
	TArray<FCacheGetChunkRequest> DDCRequests;
	DDCRequests.Reserve(TileRangesToStream.Num());
	TArray<FCacheGetChunkRequest> DDCRequestsBlocking;
	DDCRequestsBlocking.Reserve(TileRangesToStream.Num());
#endif

	// Indicies into requests we should block on after sending them all
	TArray<int32> BlockingDiskRequests;

	// Process all tile ranges selected for streaming, allocate a slot in the tile data texture for every tile and finally create IO requests for every range.
	for (FTileRange& TileRange : TileRangesToStream)
	{
		FStreamingInfo* SVTInfo = FindStreamingInfo(TileRange.SVTHandle);
		FFrameInfo& FrameInfo = SVTInfo->PerFrameInfo[TileRange.FrameIndex];
		const FResources* Resources = FrameInfo.Resources;
		FTileDataTexture* TileDataTexture = SVTInfo->TileDataTexture.Get();
		const int32 NumFrames = SVTInfo->PerFrameInfo.Num();

		// Allocate tiles
		uint32 NumSuccessfulAllocations = 0;
		bool bOutOfRequestSlots = false;
		for (uint32 i = 0; i < TileRange.TileCount; ++i)
		{
			if (bOutOfRequestSlots)
			{
				break;
			}

			const uint32 TileIndex = TileRange.TileOffset + i;
			check(FrameInfo.TileAllocations[TileIndex] == 0);

			// Allocate tile
			const uint32 TilePriority = FrameInfo.ResidentTiles.Num() - TileIndex; // Higher value means higher priority, but we want higher tiles to be streamed out first
			FTileAllocator::FAllocation PreviousAllocation;
			const uint32 TileCoord = SVTInfo->TileAllocator.Allocate(UpdateIndex, 0 /*FreeThreshold*/, TileRange.FrameIndex, TileIndex, TilePriority, false /*bLocked*/, PreviousAllocation);
			
			// Allocation failed, cut the TileRange short and upload what we have.
			if (TileCoord == INDEX_NONE)
			{
				if (GSVTStreamingLogVerbosity > 1)
				{
					UE_LOGF(LogSparseVolumeTextureStreamingManager, Warning, "Streaming pool of SVT '%ls' is oversubscribed! Tried to allocate %u tiles for frame %u, but only %u were available.",
						*SVTInfo->SVTName.ToString(), TileRange.TileCount, TileRange.FrameIndex, NumSuccessfulAllocations);
				}
				break;
			}
			++NumSuccessfulAllocations;
			FrameInfo.TileAllocations[TileIndex] = TileCoord;

			// Slot in the tile data texture was previously used by a different tile -> mark it as no longer resident/streaming.
			if (PreviousAllocation.bIsAllocated)
			{
				check(SVTInfo->PerFrameInfo.IsValidIndex(PreviousAllocation.FrameIndex));
				check(SVTInfo->PerFrameInfo[PreviousAllocation.FrameIndex].ResidentTiles.IsValidIndex(PreviousAllocation.TileIndexInFrame));
				FFrameInfo& FrameInfoTmp = SVTInfo->PerFrameInfo[PreviousAllocation.FrameIndex];

				// Mark the tile as no longer resident/streaming and make sure to set its entry in TileAllocations to 0 (for validation purposes).
				check(FrameInfoTmp.StreamingTiles[PreviousAllocation.TileIndexInFrame]);
				check(FrameInfoTmp.TileAllocations[PreviousAllocation.TileIndexInFrame] != 0);
				FrameInfoTmp.StreamingTiles[PreviousAllocation.TileIndexInFrame] = false;
				FrameInfoTmp.ResidentTiles[PreviousAllocation.TileIndexInFrame] = false;
				FrameInfoTmp.TileAllocations[PreviousAllocation.TileIndexInFrame] = 0;

				// Cancel potential IO request
				if (uint32* PendingRequestIndexPtr = FrameInfoTmp.TileIndexToPendingRequestIndex.Find((uint32)PreviousAllocation.TileIndexInFrame))
				{
					FPendingRequest& PendingRequest = PendingRequests[*PendingRequestIndexPtr];
					LOCK_PENDING_REQUEST(PendingRequest);
					check(PendingRequest.IsValid());
					if (FrameInfoTmp.StreamingTiles.CountSetBits(PendingRequest.TileOffset, PendingRequest.TileOffset + PendingRequest.TileCount) == 0)
					{
						// None of the tiles in the tile range of the IO request is still requested, so we can cancel the request
						PendingRequest.Reset();

						if (GSVTStreamingLogVerbosity > 2)
						{
							UE_LOGF(LogSparseVolumeTextureStreamingManager, Warning, "SVT '%ls' frame %i: Cancelled IO request for frame %i, tile index %i.",
								*SVTInfo->SVTName.ToString(), TileRange.FrameIndex, PreviousAllocation.FrameIndex, PreviousAllocation.TileIndexInFrame);
						}
					}

					// Remove the tile -> IO request mapping
					FrameInfoTmp.TileIndexToPendingRequestIndex.Remove((uint32)PreviousAllocation.TileIndexInFrame);
				}

				// We changed the tile residency of this frame, so add it to the set of invalidated frames that need a page table fixup
				InvalidatedSVTFrames.Add(&FrameInfoTmp);
			}
		}

		TileRange.TileCount = NumSuccessfulAllocations;
		if (TileRange.TileCount == 0)
		{
			continue;
		}

		// Create request
		{
			auto HandleRequestAllocationFailure = [&](uint32 FirstTileIndex, uint32 LastTileIndex)
			{
				// Free allocated tile data texture slots of all the tiles for which we haven't issued a request yet.
				for (uint32 TileIndexToFree = FirstTileIndex; TileIndexToFree < LastTileIndex; ++TileIndexToFree)
				{
					SVTInfo->TileAllocator.Free(FrameInfo.TileAllocations[TileIndexToFree]);
					FrameInfo.TileAllocations[TileIndexToFree] = 0;
				}
				bOutOfRequestSlots = true;
				UE_LOGF(LogSparseVolumeTextureStreamingManager, Error, "Ran out of SparseVolumeTexture IO request slots (%i)! r.SparseVolumeTexture.Streaming.MaxPendingRequests must be increased to fix this issue.", MaxPendingRequests);
			};

			const FByteBulkData& BulkData = Resources->StreamableMipLevels;
			const bool bBlockingRequest = GSVTStreamingForceBlockingRequests || (TileRange.Priority == uint8(-1));
#if WITH_EDITORONLY_DATA
			const bool bDiskRequest = (!(Resources->ResourceFlags & EResourceFlag_StreamingDataInDDC) && !BulkData.IsBulkDataLoaded());
#else
			const bool bDiskRequest = true;
#endif

#if WITH_EDITORONLY_DATA
			// When streaming from DDC, we have a slightly more complicated setup where the data is spread over multiple DDC chunks, so we likely need to issue multiple requests
			if (!bDiskRequest && (Resources->ResourceFlags & EResourceFlag_StreamingDataInDDC))
			{
				// Iterate over all chunks to find the range of chunks storing the requested range of tiles and issue a DDC request for each chunk.
				uint32 FirstTileIndexInChunk = Resources->StreamingMetaData.FirstStreamingTileIndex;
				uint32 FirstTileIndexToRead = TileRange.TileOffset;
				uint32 NumRemainingTiles = TileRange.TileCount;

				const int32 NumChunks = Resources->DDCChunkMaxTileIndices.Num();
				for (int32 ChunkIndex = 0; ChunkIndex < NumChunks && NumRemainingTiles > 0; ++ChunkIndex)
				{
					// Does this chunk overlap the requested tile range?
					const uint32 LastTileIndexInChunkPlusOne = Resources->DDCChunkMaxTileIndices[ChunkIndex] + 1;
					if (FirstTileIndexToRead >= FirstTileIndexInChunk && FirstTileIndexToRead < LastTileIndexInChunkPlusOne)
					{
						// We can only read within a single chunk per request, so we need to limit the number of tiles in this request.
						const uint32 ChunkNumReadTiles = FMath::Min(FirstTileIndexToRead + NumRemainingTiles, LastTileIndexInChunkPlusOne) - FirstTileIndexToRead;

						// Allocate and fill out a FPendingRequest.
						const int32 PendingRequestIndex = AllocatePendingRequestIndex();
						if (!ensure(PendingRequestIndex != INDEX_NONE)) // Handle allocation failure
						{
							HandleRequestAllocationFailure(FirstTileIndexToRead, FirstTileIndexToRead + NumRemainingTiles);
							break;
						}
						FPendingRequest& PendingRequest = PendingRequests[PendingRequestIndex];
						LOCK_PENDING_REQUEST(PendingRequest);
						PendingRequest.Reset();
						PendingRequest.Set(TileRange.SVTHandle, TileRange.FrameIndex, FirstTileIndexToRead, ChunkNumReadTiles, UpdateIndex, bBlockingRequest);
						PendingRequest.State = FPendingRequest::EState::DDC_Pending;

						// Create the DDC request.
						const FCacheGetChunkRequest DDCRequest = BuildDDCRequest(*Resources, PendingRequest.TileOffset, PendingRequest.TileCount, PendingRequestIndex, ChunkIndex);
						new (PendingRequest.bBlocking ? DDCRequestsBlocking : DDCRequests) FCacheGetChunkRequest(DDCRequest);

						check(ChunkNumReadTiles <= NumRemainingTiles);
						NumRemainingTiles -= ChunkNumReadTiles;
						FirstTileIndexToRead += ChunkNumReadTiles;

						// Link the streaming tiles with the request.
						for (uint32 TileIndex = PendingRequest.TileOffset; TileIndex < (PendingRequest.TileOffset + PendingRequest.TileCount); ++TileIndex)
						{
							FrameInfo.TileIndexToPendingRequestIndex.Add(TileIndex, (uint32)PendingRequestIndex);
						}

						// Mark tiles as streaming. Once they've actually been loaded into GPU memory, they'll also be marked as resident.
						FrameInfo.StreamingTiles.SetRange(PendingRequest.TileOffset, PendingRequest.TileCount, true);
					}
					FirstTileIndexInChunk = LastTileIndexInChunkPlusOne;
				}
			}
			// Otherwise we can use a single request to fetch all the tiles.
			else
#endif
			{
				// Allocate and fill out a FPendingRequest.
				const int32 PendingRequestIndex = AllocatePendingRequestIndex();
				if (!ensure(PendingRequestIndex != INDEX_NONE)) // Handle allocation failure
				{
					HandleRequestAllocationFailure(TileRange.TileOffset, TileRange.TileOffset + TileRange.TileCount);
					continue;
				}
				FPendingRequest& PendingRequest = PendingRequests[PendingRequestIndex];
				LOCK_PENDING_REQUEST(PendingRequest);
				PendingRequest.Reset();
				PendingRequest.Set(TileRange.SVTHandle, TileRange.FrameIndex, TileRange.TileOffset, TileRange.TileCount, UpdateIndex, bBlockingRequest);

#if WITH_EDITORONLY_DATA
				if (!bDiskRequest)
				{
					PendingRequest.State = FPendingRequest::EState::Memory;
				}
				else
#endif
				{
					uint32 ReadOffset = 0;
					uint32 ReadSize = 0;
					Resources->StreamingMetaData.GetTileRangeMemoryOffsetSize(TileRange.TileOffset, TileRange.TileCount, ReadOffset, ReadSize);

					PendingRequest.RequestBuffer = FIoBuffer(ReadSize); // SVT_TODO: Use FIoBuffer::Wrap with preallocated memory
					const EAsyncIOPriorityAndFlags Priority = PendingRequest.bBlocking ? AIOP_CriticalPath : AIOP_Low;
					// SVT_TODO: We're currently using a single batch per request so we can individually cancel and wait on requests.
					// This isn't ideal and should be revisited in the future.
					TRACE_IOSTORE_METADATA_SCOPE_TAG("SparseVolumeTexture");
					FBulkDataBatchRequest::FScatterGatherBuilder Batch = FBulkDataBatchRequest::ScatterGather(1);
					Batch.Read(BulkData, ReadOffset, ReadSize);
					Batch.Issue(PendingRequest.RequestBuffer, Priority, [](FBulkDataRequest::EStatus){}, PendingRequest.Request);

					if (PendingRequest.bBlocking)
					{
						BlockingDiskRequests.Add(PendingRequestIndex);
					}

#if WITH_EDITORONLY_DATA
					PendingRequest.State = FPendingRequest::EState::Disk;
#endif
				}

				// Link the streaming tiles with the request.
				for (uint32 TileIndex = PendingRequest.TileOffset; TileIndex < (PendingRequest.TileOffset + PendingRequest.TileCount); ++TileIndex)
				{
					FrameInfo.TileIndexToPendingRequestIndex.Add(TileIndex, (uint32)PendingRequestIndex);
				}

				// Mark tiles as streaming. Once they've actually been loaded into GPU memory, they'll also be marked as resident.
				FrameInfo.StreamingTiles.SetRange(PendingRequest.TileOffset, PendingRequest.TileCount, true);
			}
		}
	}

	// Wait on all blocking disk requests
	for (int32 PendingRequestIndex : BlockingDiskRequests)
	{
		FPendingRequest& PendingRequest = PendingRequests[PendingRequestIndex];
		LOCK_PENDING_REQUEST(PendingRequest);
		PendingRequest.Request.Wait();
	}

	// Now we can finally issue the requests
	{
#if WITH_EDITORONLY_DATA
		if (!DDCRequests.IsEmpty())
		{
			RequestDDCData(DDCRequests, false /*bBlocking*/);
			DDCRequests.Empty();
		}
		if (!DDCRequestsBlocking.IsEmpty())
		{
			RequestDDCData(DDCRequestsBlocking, true /*bBlocking*/);
			DDCRequestsBlocking.Empty();
		}
#endif
	}
}

int32 FStreamingManager::DetermineReadyRequests()
{
	using namespace UE::DerivedData;

	TRACE_CPUPROFILER_EVENT_SCOPE(SVT::StreamingDetermineReadyRequests);

	const int32 StartPendingRequestIndex = (NextPendingRequestIndex + MaxPendingRequests - NumPendingRequests) % MaxPendingRequests;
	int32 NumReadyRequests = 0;

	for (int32 i = 0; i < NumPendingRequests; ++i)
	{
		const int32 PendingRequestIndex = (StartPendingRequestIndex + i) % MaxPendingRequests;
		FPendingRequest& PendingRequest = PendingRequests[PendingRequestIndex];
		LOCK_PENDING_REQUEST(PendingRequest);

		// Check if request was cancelled
		if (!PendingRequest.IsValid())
		{
#if WITH_EDITORONLY_DATA
			// Just mark as ready so it will be skipped later
			PendingRequest.State = FPendingRequest::EState::DDC_Ready;
#endif
			++NumReadyRequests; // Don't forget to increment in this case too or we might miss some ready requests later!
			continue; 
		}
		FStreamingInfo* SVTInfo = FindStreamingInfo(PendingRequest.SVTHandle);
		check(SVTInfo);
		const FResources* Resources = SVTInfo->PerFrameInfo[PendingRequest.FrameIndex].Resources;

#if WITH_EDITORONLY_DATA
		if (PendingRequest.State == FPendingRequest::EState::DDC_Ready)
		{
			if (PendingRequest.RetryCount > 0 && GSVTStreamingLogVerbosity > 0)
			{
				check(SVTInfo);
				UE_LOGF(LogSparseVolumeTextureStreamingManager, Display, "SVT DDC retry succeeded for '%ls' (frame %i, tile offset %i, tile count %i) on %i attempt.",
					*SVTInfo->SVTName.ToString(), PendingRequest.FrameIndex, PendingRequest.TileOffset, PendingRequest.TileCount, PendingRequest.RetryCount);
			}
		}
		else if (PendingRequest.State == FPendingRequest::EState::DDC_Pending)
		{
			break;
		}
		else if (PendingRequest.State == FPendingRequest::EState::DDC_Failed)
		{
			PendingRequest.State = FPendingRequest::EState::DDC_Pending;

			if (GSVTStreamingLogVerbosity > 1 || (GSVTStreamingLogVerbosity > 0 && PendingRequest.RetryCount == 0)) // Only warn on first retry to prevent spam
			{
				UE_LOGF(LogSparseVolumeTextureStreamingManager, Warning, "SVT DDC request failed for '%ls' (frame %i, tile offset %i, tile count %i). Retrying...",
					*SVTInfo->SVTName.ToString(), PendingRequest.FrameIndex, PendingRequest.TileOffset, PendingRequest.TileCount);
			}

			const int32 NumChunks = Resources->DDCChunkMaxTileIndices.Num();
			uint32 FirstTileIndexInChunk = Resources->StreamingMetaData.FirstStreamingTileIndex;
			bool bFoundChunk = false;
			for (int32 ChunkIndex = 0; ChunkIndex < NumChunks; ++ChunkIndex)
			{
				const uint32 LastTileIndexInChunkPlusOne = Resources->DDCChunkMaxTileIndices[ChunkIndex] + 1;
				if (PendingRequest.TileOffset >= FirstTileIndexInChunk && (PendingRequest.TileOffset + PendingRequest.TileCount) <= LastTileIndexInChunkPlusOne)
				{
					bFoundChunk = true;
					const FCacheGetChunkRequest Request = BuildDDCRequest(*Resources, PendingRequest.TileOffset, PendingRequest.TileCount, PendingRequestIndex, ChunkIndex);
					const bool bBlocking = GSVTStreamingForceBlockingRequests || PendingRequest.bBlocking;
					RequestDDCData(MakeArrayView(&Request, 1), bBlocking);
					break;
				}
			}
			check(bFoundChunk);
			++PendingRequest.RetryCount;
			break;
		}
		else if (PendingRequest.State == FPendingRequest::EState::Memory)
		{
			// Memory is always ready
		}
		else
#endif // WITH_EDITORONLY_DATA
		{
#if WITH_EDITORONLY_DATA
			check(PendingRequest.State == FPendingRequest::EState::Disk);
#endif
			if (PendingRequest.Request.IsCompleted())
			{
				if (!PendingRequest.Request.IsOk())
				{
					// Retry if IO request failed for some reason
					uint32 ReadOffset = 0;
					uint32 ReadSize = 0;
					Resources->StreamingMetaData.GetTileRangeMemoryOffsetSize(PendingRequest.TileOffset, PendingRequest.TileCount, ReadOffset, ReadSize);
					if (GSVTStreamingLogVerbosity > 0)
					{
						UE_LOGF(LogSparseVolumeTextureStreamingManager, Warning, "SVT IO request failed for %ls (frame %i, tile offset %i, tile count %i, offset %i, size %i). Retrying...",
							*SVTInfo->SVTName.ToString(), PendingRequest.FrameIndex, PendingRequest.TileOffset, PendingRequest.TileCount, ReadOffset, ReadSize);
					}
					
					TRACE_IOSTORE_METADATA_SCOPE_TAG("SparseVolumeTexture (retry)");
					FBulkDataBatchRequest::FScatterGatherBuilder Batch = FBulkDataBatchRequest::ScatterGather(1);
					Batch.Read(Resources->StreamableMipLevels, ReadOffset, ReadSize);
					Batch.Issue(PendingRequest.RequestBuffer, AIOP_Low, [](FBulkDataRequest::EStatus) {}, PendingRequest.Request);
					break;
				}
			}
			else
			{
				break;
			}
		}

		++NumReadyRequests;
	}

	return NumReadyRequests;
}

void FStreamingManager::InstallReadyRequests()
{
	check(AsyncState.bUpdateActive);
	check(AsyncState.NumReadyRequests <= PendingRequests.Num());
	if (AsyncState.NumReadyRequests <= 0)
	{
		return;
	}

	TRACE_CPUPROFILER_EVENT_SCOPE(SVT::StreamingInstallReadyRequests);

	UploadTasks.Reset();
	UploadTasks.Reserve(AsyncState.NumReadyRequests * 2 /*slack for splitting large uploads*/);

#if WITH_EDITORONLY_DATA
	TMap<const FResources*, const uint8*> ResourceToBulkPointer;
#endif

	// Do a second pass over all ready requests, claiming memory in the upload buffers and creating FUploadTasks
	const int32 StartPendingRequestIndex = (NextPendingRequestIndex + MaxPendingRequests - NumPendingRequests) % MaxPendingRequests;
	for (int32 i = 0; i < AsyncState.NumReadyRequests; ++i)
	{
		const int32 PendingRequestIndex = (StartPendingRequestIndex + i) % MaxPendingRequests;
		FPendingRequest& PendingRequest = PendingRequests[PendingRequestIndex];
		LOCK_PENDING_REQUEST(PendingRequest);

		// Skip if request was cancelled
		if (!PendingRequest.IsValid())
		{
			continue;
		}

		FStreamingInfo* SVTInfo = FindStreamingInfo(PendingRequest.SVTHandle);
		check(SVTInfo);
		const int32 FormatSizeA = GPixelFormats[SVTInfo->FormatA].BlockBytes;
		const int32 FormatSizeB = GPixelFormats[SVTInfo->FormatB].BlockBytes;

		FFrameInfo& FrameInfo = SVTInfo->PerFrameInfo[PendingRequest.FrameIndex];
		const FResources* Resources = FrameInfo.Resources;
		const FPageTopology& Topology = FrameInfo.Resources->Topology;
		uint32 ReadOffset = 0;
		uint32 ReadSize = 0;
		Resources->StreamingMetaData.GetTileRangeMemoryOffsetSize(PendingRequest.TileOffset, PendingRequest.TileCount, ReadOffset, ReadSize);

		const uint8* SrcPtr = nullptr;
		const uint8* SrcEndPtr = nullptr;

#if WITH_EDITORONLY_DATA
		if (PendingRequest.State == FPendingRequest::EState::DDC_Ready)
		{
			check(Resources->ResourceFlags & EResourceFlag_StreamingDataInDDC);
			SrcPtr = (const uint8*)PendingRequest.SharedBuffer.GetData();
			SrcEndPtr = SrcPtr + PendingRequest.SharedBuffer.GetSize();
		}
		else if (PendingRequest.State == FPendingRequest::EState::Memory)
		{
			const uint8** BulkDataPtrPtr = ResourceToBulkPointer.Find(Resources);
			if (BulkDataPtrPtr)
			{
				SrcPtr = *BulkDataPtrPtr + ReadOffset;
				SrcEndPtr = SrcPtr + ReadSize;
			}
			else
			{
				const FByteBulkData& BulkData = Resources->StreamableMipLevels;
				check(BulkData.IsBulkDataLoaded() && BulkData.GetBulkDataSize() > 0);
				const uint8* BulkDataPtr = (const uint8*)BulkData.LockReadOnly();
				ResourceToBulkPointer.Add(Resources, BulkDataPtr);
				SrcPtr = BulkDataPtr + ReadOffset;
				SrcEndPtr = BulkDataPtr + BulkData.GetBulkDataSize();
			}
		}
		else
#endif
		{
#if WITH_EDITORONLY_DATA
			check(PendingRequest.State == FPendingRequest::EState::Disk);
#endif
			SrcPtr = PendingRequest.RequestBuffer.GetData();
			SrcEndPtr = SrcPtr + PendingRequest.RequestBuffer.DataSize();
		}

		for (uint32 TileIndex = PendingRequest.TileOffset; TileIndex < (PendingRequest.TileOffset + PendingRequest.TileCount); ++TileIndex)
		{
			SVTInfo->PerFrameInfo[PendingRequest.FrameIndex].TileIndexToPendingRequestIndex.Remove(TileIndex);
			if (!SVTInfo->PerFrameInfo[PendingRequest.FrameIndex].StreamingTiles[TileIndex])
			{
				continue; // Skip tile install. Tile was "streamed out" before it was even installed in the first place.
			}

			const FTileInfo TileInfo = Resources->StreamingMetaData.GetTileInfo(TileIndex, FormatSizeA, FormatSizeB);

			check(SrcPtr);
			check(TileInfo.Offset >= ReadOffset);
			const uint8* TileSrcPtr = SrcPtr + (TileInfo.Offset - ReadOffset);
			check((TileSrcPtr + TileInfo.Size) <= SrcEndPtr);

			FTileUploader::FAddResult TileDataAddResult = SVTInfo->TileDataTexture->AddUpload(1, TileInfo.NumVoxels[0], TileInfo.NumVoxels[1]);

			FTileDataTask& TileDataTask = UploadTasks.AddDefaulted_GetRef();
			TileDataTask.DstOccupancyBitsPtrs = TileDataAddResult.OccupancyBitsPtrs;
			TileDataTask.DstTileDataOffsetsPtrs = TileDataAddResult.TileDataOffsetsPtrs;
			TileDataTask.DstTileDataPtrs = TileDataAddResult.TileDataPtrs;
			TileDataTask.DstPhysicalTileCoordsPtr = TileDataAddResult.PackedPhysicalTileCoordsPtr;
			TileDataTask.SrcOccupancyBitsPtrs[0] = TileSrcPtr + TileInfo.OccupancyBitsOffsets[0];
			TileDataTask.SrcOccupancyBitsPtrs[1] = TileSrcPtr + TileInfo.OccupancyBitsOffsets[1];
			TileDataTask.SrcVoxelDataPtrs[0] = TileSrcPtr + TileInfo.VoxelDataOffsets[0];
			TileDataTask.SrcVoxelDataPtrs[1] = TileSrcPtr + TileInfo.VoxelDataOffsets[1];
			TileDataTask.VoxelDataSizes = TileInfo.VoxelDataSizes;
			TileDataTask.VoxelDataBaseOffsets = TileDataAddResult.TileDataBaseOffsets;
			TileDataTask.PhysicalTileCoord = FrameInfo.TileAllocations[TileIndex];

			FrameInfo.ResidentTiles[TileIndex] = true;
		}

		InvalidatedSVTFrames.Add(&FrameInfo);
	}

	// Do all the memcpys in parallel
	ParallelFor(TEXT("SVT::UploadTileDataTasks"), UploadTasks.Num(), 8, [&](int32 TaskIndex)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(SVT::StreamingTileDataUpload);

		FTileDataTask& TileDataTask = UploadTasks[TaskIndex];
		for (int32 TexIdx = 0; TexIdx < 2; ++TexIdx)
		{
			if (TileDataTask.DstOccupancyBitsPtrs[TexIdx])
			{
				FMemory::Memcpy(TileDataTask.DstOccupancyBitsPtrs[TexIdx], TileDataTask.SrcOccupancyBitsPtrs[TexIdx], SVT::NumOccupancyWordsPerPaddedTile * sizeof(uint32));
			}
			if (TileDataTask.VoxelDataSizes[TexIdx] > 0)
			{
				FMemory::Memcpy(TileDataTask.DstTileDataPtrs[TexIdx], TileDataTask.SrcVoxelDataPtrs[TexIdx], TileDataTask.VoxelDataSizes[TexIdx]);
			}
			if (TileDataTask.DstTileDataOffsetsPtrs[TexIdx])
			{
				FMemory::Memcpy(TileDataTask.DstTileDataOffsetsPtrs[TexIdx], &TileDataTask.VoxelDataBaseOffsets[TexIdx], sizeof(uint32));
			}
		}
		FMemory::Memcpy(TileDataTask.DstPhysicalTileCoordsPtr, &TileDataTask.PhysicalTileCoord, sizeof(uint32));
	});

	// Clean up requests
	for (int32 i = 0; i < AsyncState.NumReadyRequests; ++i)
	{
		const int32 PendingRequestIndex = (StartPendingRequestIndex + i) % MaxPendingRequests;
		FPendingRequest& PendingRequest = PendingRequests[PendingRequestIndex];
		LOCK_PENDING_REQUEST(PendingRequest);
		PendingRequest.Reset();
	}

#if WITH_EDITORONLY_DATA
	// Unlock BulkData
	for (auto& Pair : ResourceToBulkPointer)
	{
		Pair.Key->StreamableMipLevels.Unlock();
	}
#endif
}

void FStreamingManager::PatchPageTable(FRDGBuilder& GraphBuilder)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(SVT::StreamingPatchPageTable);

	int32 NumUpdates = 0;

	// Generate bitsets of invalidated pages for every frame.
	for (FFrameInfo* FrameInfoPtr : InvalidatedSVTFrames)
	{
		FFrameInfo& FrameInfo = *FrameInfoPtr;
		const FPageTopology& Topology = FrameInfo.Resources->Topology;
		const uint32 NumPages = Topology.NumPages();

		// Mark all pages with resident tiles as resident. For now we also enforce that a page is only resident if its parent is resident, but in theory this is not needed.
		ResidentPagesNew.SetNumUninitialized(NumPages);
		ResidentPagesNew.SetRange(0, NumPages, false);
		for (uint32 PageIndex = 0; PageIndex < NumPages; ++PageIndex)
		{
			const uint32 ParentIndex = Topology.ParentIndices[PageIndex];
			const uint32 TileIndex = Topology.TileIndices[PageIndex];
			ResidentPagesNew[PageIndex] = (ParentIndex == INDEX_NONE || ResidentPagesNew[ParentIndex]) && FrameInfo.ResidentTiles[TileIndex];
		}

		// Get all pages that were streamed in or out in this streaming update by doing a bitwise XOR with the resident pages from before the current streaming update.
		ResidentPagesDiff = FrameInfo.ResidentPages;
		ResidentPagesDiff.CombineWithBitwiseXOR(ResidentPagesNew, EBitwiseOperatorFlags::MaxSize);
		
		// Initialize InvalidatedPages with the diff. In the following loop, we then find all descendants of the newly streamed in/out pages and also mark them as invalidated.
		FrameInfo.InvalidatedPages = ResidentPagesDiff;
		// Iterate over all pages that were NOT changed in this update. We then dig down into their parents and try to figure out if they were newly streamed in/out. If so, mark the current page as also invalidated.
		ResidentPagesDiff.BitwiseNOT();
		for (TConstSetBitIterator It(ResidentPagesDiff); It; ++It)
		{
			const int32 PageIndex = It.GetIndex();
			check(Topology.IsValidPageIndex(PageIndex));

			// Only try to invalidate this page if it is not already resident in GPU memory
			if (!FrameInfo.InvalidatedPages[PageIndex] && !FrameInfo.ResidentPages[PageIndex])
			{
				uint32 ParentPageIndex = Topology.ParentIndices[PageIndex];
				while (ParentPageIndex != INDEX_NONE)
				{
					if (FrameInfo.InvalidatedPages[ParentPageIndex])
					{
						FrameInfo.InvalidatedPages[PageIndex] = true;
						break;
					}
					ParentPageIndex = Topology.ParentIndices[ParentPageIndex];
				}
			}
		}

		// Update the bitset of resident pages
		FrameInfo.ResidentPages = ResidentPagesNew;
		
		NumUpdates += FrameInfo.InvalidatedPages.CountSetBits();
	}

	if (NumUpdates > 0)
	{
		PageTableUpdater->Init(GraphBuilder, NumUpdates, 0);

		// Generate updates
		for (FFrameInfo* FrameInfoPtr : InvalidatedSVTFrames)
		{
			FFrameInfo& FrameInfo = *FrameInfoPtr;
			const FPageTopology& Topology = FrameInfo.Resources->Topology;
			
			// This set of variables is updated every time we start processing a new mip level
			bool bEnteredNewMipRange = true;
			int32 MipLevel = FrameInfo.NumMipLevels - 1;
			int32 NumUpdatesThisMip = 0;
			uint32 MipUpdateWriteIndex = 0;
			uint8* DstCoordsPtr = nullptr;
			uint8* DstEntryPtr = nullptr;

			// Iterate over all invalidated pages and generate page table updates (packed page write coord and data to write to that coord).
			for (TConstSetBitIterator It(FrameInfo.InvalidatedPages); It; ++It)
			{
				const int32 PageIndex = It.GetIndex();
				check(Topology.IsValidPageIndex(PageIndex));

				auto IsInMipRange = [](const FPageTopology& InTopology, uint32 InIndex, int32 InMipLevel)
				{
					return InIndex >= InTopology.MipInfo[InMipLevel].PageOffset && InIndex < (InTopology.MipInfo[InMipLevel].PageOffset + InTopology.MipInfo[InMipLevel].PageCount);
				};

				// Determine the current mip level. Bits are ordered highest to lowest mip level.
				while (MipLevel > 0 && !IsInMipRange(Topology, PageIndex, MipLevel))
				{
					bEnteredNewMipRange = true;
					--MipLevel;
					check(MipLevel >= 0);
				}
				check(IsInMipRange(Topology, PageIndex, MipLevel));

				// If we entered a new mip range, get a new set of write pointers from the PageTableUpdater.
				if (bEnteredNewMipRange)
				{
					check(NumUpdatesThisMip == MipUpdateWriteIndex);
					const uint32 PageOffset = Topology.MipInfo[MipLevel].PageOffset;
					const uint32 PageCount = Topology.MipInfo[MipLevel].PageCount;
					NumUpdatesThisMip = FrameInfo.InvalidatedPages.CountSetBits(PageOffset, PageOffset + PageCount);
					MipUpdateWriteIndex = 0;
					bEnteredNewMipRange = false;

					PageTableUpdater->Add_GetRef(FrameInfo.PageTableTexture, MipLevel, NumUpdatesThisMip, DstCoordsPtr, DstEntryPtr);
				}

				// Get the tile data texture coordinate we need to write into the GPU page table texture.
				uint32 PageTableEntry = 0;
				if (FrameInfo.ResidentPages[PageIndex])
				{
					// This page is already resident, so we can simply use its value from TileAllocations
					PageTableEntry = FrameInfo.TileAllocations[Topology.TileIndices[PageIndex]];
					check(PageTableEntry);
					PageTableEntry |= MipLevel << 24u;
				}
				else
				{
					// This page is not resident but needs a fallback value written to it, so we probe the parent pages until we find a resident one
					uint32 ParentPageIndex = Topology.ParentIndices[PageIndex];
					int32 ParentMipLevel = MipLevel + 1;
					while (ParentPageIndex != INDEX_NONE)
					{
						// The parent page is resident in GPU memory, so we can use it's cached value in PageEntries.
						if (FrameInfo.ResidentPages[ParentPageIndex])
						{
							PageTableEntry = FrameInfo.TileAllocations[Topology.TileIndices[ParentPageIndex]];
							check(PageTableEntry);
							PageTableEntry |= ParentMipLevel << 24u;
							break;
						}
						ParentPageIndex = Topology.ParentIndices[ParentPageIndex];
						++ParentMipLevel;
					}
					check(ParentPageIndex != INDEX_NONE); // If we hit this, then we tried to find the root node's parent. This should never happen as the root node should always be resident.
				}

				// Write the update to the upload buffer pointers
				reinterpret_cast<uint32*>(DstCoordsPtr)[MipUpdateWriteIndex] = Topology.PackedPageTableCoords[PageIndex];
				reinterpret_cast<uint32*>(DstEntryPtr)[MipUpdateWriteIndex] = PageTableEntry;
				++MipUpdateWriteIndex;
			}
		}

		PageTableUpdater->Apply(GraphBuilder);
	}

	InvalidatedSVTFrames.Reset();
}

FStreamingManager::FStreamingInfo* FStreamingManager::FindStreamingInfo(uint16 SparseVolumeTextureHandle)
{
	if (!StreamingInfo.IsValidIndex(SparseVolumeTextureHandle) || !StreamingInfo.IsAllocated(SparseVolumeTextureHandle))
	{
		return nullptr;
	}
	TUniquePtr<FStreamingInfo>& SVTInfo = StreamingInfo[SparseVolumeTextureHandle];
	return SVTInfo.Get();
}

FStreamingManager::FStreamingInfo* FStreamingManager::FindStreamingInfo(UStreamableSparseVolumeTexture* SparseVolumeTexture)
{
	if (uint16* HandlePtr = SparseVolumeTextureToHandle.Find(SparseVolumeTexture))
	{
		return FindStreamingInfo(*HandlePtr);
	}
	return nullptr;
}

int32 FStreamingManager::AllocatePendingRequestIndex()
{
	if (NumPendingRequests < MaxPendingRequests)
	{
		int32 Result = NextPendingRequestIndex;
		NextPendingRequestIndex = (NextPendingRequestIndex + 1) % MaxPendingRequests;
		++NumPendingRequests;
		return Result;
	}
	else
	{
		return INDEX_NONE;
	}
}

#if WITH_EDITORONLY_DATA

UE::DerivedData::FCacheGetChunkRequest FStreamingManager::BuildDDCRequest(const FResources& Resources, uint32 FirstTileIndex, uint32 NumTiles, uint32 PendingRequestIndex, int32 ChunkIndex)
{
	using namespace UE::DerivedData;

	const uint32 FirstTileIndexInChunk = ChunkIndex > 0 ? (Resources.DDCChunkMaxTileIndices[ChunkIndex - 1] + 1) : Resources.StreamingMetaData.FirstStreamingTileIndex;
	const uint32 LastTileIndexInChunkPlusOne = Resources.DDCChunkMaxTileIndices[ChunkIndex] + 1;
	check(FirstTileIndex >= FirstTileIndexInChunk);
	check((FirstTileIndex + NumTiles) <= LastTileIndexInChunkPlusOne);
	const uint32 ReadOffsetInChunk = Resources.StreamingMetaData.TileDataOffsets[FirstTileIndex] - Resources.StreamingMetaData.TileDataOffsets[FirstTileIndexInChunk];
	const uint32 ReadSizeInChunk = Resources.StreamingMetaData.TileDataOffsets[FirstTileIndex + NumTiles] - Resources.StreamingMetaData.TileDataOffsets[FirstTileIndex];
	const uint32 ChunkTotalSize = Resources.StreamingMetaData.TileDataOffsets[LastTileIndexInChunkPlusOne] - Resources.StreamingMetaData.TileDataOffsets[FirstTileIndexInChunk];

	checkf(ReadSizeInChunk > 0, TEXT("DDC chunk request sizes must be greater than zero."));

	FCacheGetChunkRequest Request;
	Request.Id = FValueId(FMemoryView(Resources.DDCChunkIds[ChunkIndex].GetData(), 12));
	Request.Key.Bucket = FCacheBucket(TEXT("SparseVolumeTexture"));
	Request.Key.Hash = Resources.DDCKeyHash;
	if (ReadOffsetInChunk != 0 || ReadSizeInChunk != ChunkTotalSize)
	{
		Request.RawOffset = ReadOffsetInChunk;
		Request.RawSize = ReadSizeInChunk;
	}
	Request.UserData = (((uint64)PendingRequestIndex) << uint64(32)) | (uint64)PendingRequests[PendingRequestIndex].RequestVersion;
	return Request;
}

void FStreamingManager::RequestDDCData(TConstArrayView<UE::DerivedData::FCacheGetChunkRequest> DDCRequests, bool bBlocking)
{
	using namespace UE::DerivedData;

	{
		FRequestOwner* RequestOwnerPtr = bBlocking ? RequestOwnerBlocking.Get() : RequestOwner.Get();
		FRequestBarrier Barrier(*RequestOwnerPtr);	// This is a critical section on the owner. It does not constrain ordering
		GetCache().GetChunks(DDCRequests, *RequestOwnerPtr,
			[this](FCacheGetChunkResponse&& Response)
			{
				const uint32 PendingRequestIndex = (uint32)(Response.UserData >> uint64(32));
				const uint32 RequestVersion = (uint32)Response.UserData;

				FPendingRequest& PendingRequest = PendingRequests[PendingRequestIndex];
				LOCK_PENDING_REQUEST(PendingRequest);

				// In case the request returned after the data was already streamed out again we need to abort so that we do not overwrite data in the FPendingRequest slot.
				if (RequestVersion != PendingRequest.RequestVersion)
				{
					return;
				}

				check(PendingRequest.IsValid());

				if (Response.Status == EStatus::Ok)
				{
					PendingRequest.SharedBuffer = MoveTemp(Response.RawData);
					PendingRequest.State = FPendingRequest::EState::DDC_Ready;
				}
				else
				{
					PendingRequest.State = FPendingRequest::EState::DDC_Failed;
				}
			});
	}
	
	if (bBlocking)
	{
		RequestOwnerBlocking->Wait();
	}
}

#endif // WITH_EDITORONLY_DATA

SIZE_T FStreamingManager::GetAllocatedSize() const
{
	SIZE_T AllocatedSize = 0;
	AllocatedSize += SparseVolumeTextureToHandle.GetAllocatedSize();
	AllocatedSize += StreamingInfo.GetAllocatedSize();
	for (const auto& Info : StreamingInfo)
	{
		AllocatedSize += Info->GetAllocatedSize();
	}
	AllocatedSize += RequestsHashTable.GetAllocatedSize();
	AllocatedSize += PendingRequests.GetAllocatedSize();
	
	return AllocatedSize;
}

FStreamingInstance* FStreamingManager::FStreamingInfo::GetAndUpdateStreamingInstance(uint64 StreamingInstanceKey, const FStreamingInstanceRequest& Request)
{
	// Try to find a FStreamingInstance around the requested frame index and matching the same key
	FStreamingInstance* StreamingInstance = nullptr;
	for (TUniquePtr<FStreamingInstance>& Instance : StreamingInstances)
	{
		if (Instance->IsFrameInWindow(Request.FrameIndex) && (StreamingInstanceKey == Instance->GetKey()))
		{
			StreamingInstance = Instance.Get();
			StreamingInstance->AddRequest(Request);
			break;
		}
	}
	// No existing instance found -> create a new one
	if (!StreamingInstance)
	{
		StreamingInstance = StreamingInstances[StreamingInstances.Emplace(MakeUnique<FStreamingInstance>(StreamingInstanceKey, PerFrameInfo.Num(), TArrayView<uint32>(MipLevelStreamingSize), Request))].Get();
	}
	return StreamingInstance;
}

SIZE_T FStreamingManager::FFrameInfo::GetAllocatedSize() const
{
	return TileAllocations.GetAllocatedSize()
		+ ResidentPages.GetAllocatedSize()
		+ InvalidatedPages.GetAllocatedSize()
		+ ResidentTiles.GetAllocatedSize()
		+ StreamingTiles.GetAllocatedSize()
		+ TileIndexToPendingRequestIndex.GetAllocatedSize();
}

SIZE_T FStreamingManager::FStreamingInfo::GetAllocatedSize() const
{
	SIZE_T AllocatedSize = 0;
	for (const FFrameInfo& FrameInfo : PerFrameInfo)
	{
		AllocatedSize += FrameInfo.GetAllocatedSize();
	}
	AllocatedSize += PerFrameInfo.GetAllocatedSize();
	AllocatedSize += StreamingInstances.GetAllocatedSize() + StreamingInstances.Num() * sizeof(*StreamingInstances[0]); // StreamingInstances only stores pointers, but each FStreamingInstance has the same size.
	AllocatedSize += MipLevelStreamingSize.GetAllocatedSize();
	AllocatedSize += TileAllocator.GetAllocatedSize();
	AllocatedSize += sizeof(*TileDataTexture);
	return AllocatedSize;
}

void FTileAllocator::Init(const FIntVector3& InResolutionInTiles)
{
	check(InResolutionInTiles.X > 0 && InResolutionInTiles.Y > 0 && InResolutionInTiles.Z > 0);
	FreeHeap.Clear();
	ResolutionInTiles = InResolutionInTiles;
	TileCapacity = ResolutionInTiles.X * ResolutionInTiles.Y * ResolutionInTiles.Z;
	NumAllocated = 1; // Implicitly allocate null tile
	Allocations.SetNumZeroed(TileCapacity);
	Allocations[0] = FAllocation(INDEX_NONE, INDEX_NONE, true, true);
	
	// Populate free heap with packed tile coordinates, ready for allocation
	for (int32 TileZ = 0; TileZ < ResolutionInTiles.Z; ++TileZ)
	{
		for (int32 TileY = 0; TileY < ResolutionInTiles.Y; ++TileY)
		{
			for (int32 TileX = 0; TileX < ResolutionInTiles.X; ++TileX)
			{
				// Skip (0,0,0): That's where we implicitly allocated the null tile
				const uint32 PackedCoord = SVT::PackPageTableEntry(FIntVector3(TileX, TileY, TileZ));
				if (PackedCoord)
				{
					FreeHeap.Add(0, PackedCoord);
				}
			}
		}
	}
}

uint32 FTileAllocator::Allocate(uint32 UpdateIndex, uint32 FreeThreshold, uint16 FrameIndex, uint32 TileIndexInFrame, uint32 TilePriority, bool bLocked, FAllocation& OutPreviousAllocation)
{
	if (FreeHeap.IsEmpty())
	{
		// No free tiles available!
		return INDEX_NONE;
	}

	const uint32 TileCoord = FreeHeap.Top();

	// There are non-locked tiles in the heap, but they've all been recently requested, so we can't reuse them for this allocation!
	const uint32 LastRequested = uint32(FreeHeap.GetKey(TileCoord) >> 32u);
	if ((LastRequested + FreeThreshold) >= UpdateIndex)
	{
		return INDEX_NONE;
	}

	const FIntVector3 UnpackedTileCoord = SVT::UnpackPageTableEntry(TileCoord);
	const uint32 LinearTileCoord = (UnpackedTileCoord.Z * ResolutionInTiles.Y * ResolutionInTiles.X) + (UnpackedTileCoord.Y * ResolutionInTiles.X) + UnpackedTileCoord.X;
	FAllocation& Allocation = Allocations[LinearTileCoord];

	// Tile was already allocated
	if (Allocation.bIsAllocated)
	{
		check(!Allocation.bIsLocked);
		OutPreviousAllocation = Allocation;
		--NumAllocated;
	}
	else
	{
		// Tile wasn't allocated, but for consistency and debugging, simply zero the out parameter.
		FMemory::Memzero(OutPreviousAllocation);
	}

	// Mark as used by the new SVT frame
	Allocation.FrameIndex = FrameIndex;
	Allocation.TileIndexInFrame = TileIndexInFrame;
	Allocation.bIsLocked = bLocked;
	Allocation.bIsAllocated = true;

	if (bLocked)
	{
		// Allocation is locked: remove it from the FreeHeap so it can't ever be reused unless manually freed.
		FreeHeap.Pop();
	}
	else
	{
		// Allocation is not locked: update it with the new key
		const uint64 HeapKey = (((uint64)UpdateIndex) << 32u) + TilePriority;
		FreeHeap.Update(HeapKey, TileCoord);
	}

	++NumAllocated;
	check(NumAllocated <= TileCapacity);

	return TileCoord;
}

void FTileAllocator::UpdateUsage(uint32 UpdateIndex, uint32 TileCoord, uint32 TilePriority)
{
	if (FreeHeap.IsPresent(TileCoord))
	{
		// Mark tile as requested in the given UpdateIndex
		const uint64 HeapKey = (((uint64)UpdateIndex) << 32u) + TilePriority;
		FreeHeap.Update(HeapKey, TileCoord);
	}
}

void FTileAllocator::Free(uint32 TileCoord)
{
	TileCoord &= PhysicalCoordMask; // Clear the upper 8 bit in case the caller wrote "user data" (mip level index) there.
	check(TileCoord);

	// Make sure the passed in TileCoord actually has a valid value.
	const FIntVector3 UnpackedTileCoord = SVT::UnpackPageTableEntry(TileCoord);
	check(IsInBounds(UnpackedTileCoord, FIntVector3::ZeroValue, ResolutionInTiles));
	
	// Zero the allocation info
	const uint32 LinearTileCoord = (UnpackedTileCoord.Z * ResolutionInTiles.Y * ResolutionInTiles.X) + (UnpackedTileCoord.Y * ResolutionInTiles.X) + UnpackedTileCoord.X;
	FMemory::Memzero(Allocations[LinearTileCoord]);

	if (FreeHeap.IsPresent(TileCoord))
	{
		// Tile was already in the heap; simply put it at the front so it'll be used in the next allocation.
		FreeHeap.Update(0, TileCoord);
	}
	else
	{
		// Tile wasn't in the heap (must have been a locked allocation), so we add it now.
		FreeHeap.Add(0, TileCoord);
	}

	check(NumAllocated > 1); // Null tile is permanently allocated
	--NumAllocated;
}

SIZE_T FTileAllocator::GetAllocatedSize() const
{
	// FBinaryHeap unfortunately does not support querying the allocated size, so we just compute it ourselves here.
	const SIZE_T HeapSize = FreeHeap.GetHeapSize() * sizeof(uint32) + FreeHeap.GetIndexSize() * (sizeof(uint64) + sizeof(uint32));
	return HeapSize + Allocations.GetAllocatedSize();
}

}
}

#undef LOCK_PENDING_REQUEST
