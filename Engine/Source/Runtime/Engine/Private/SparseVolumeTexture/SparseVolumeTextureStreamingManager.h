// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Engine/EngineTypes.h"
#include "SparseVolumeTexture/ISparseVolumeTextureStreamingManager.h"
#include "IO/IoDispatcher.h"
#include "Memory/SharedBuffer.h"
#include "Serialization/BulkData.h"
#include "RenderResource.h"
#include "Containers/Map.h"
#include "Containers/StaticArray.h"
#include "Containers/BinaryHeap.h"
#include "RenderGraphBuilder.h"
#include "Misc/DateTime.h"

DECLARE_LOG_CATEGORY_EXTERN(LogSparseVolumeTextureStreamingManager, Log, All);
DECLARE_STATS_GROUP(TEXT("SparseVolumeTexture Memory"), STATGROUP_SparseVolumeTextureMemory, STATCAT_Advanced);
DECLARE_MEMORY_STAT_POOL_EXTERN(TEXT("Total GPU Memory"), STAT_SVTGPUMemoryTotal, STATGROUP_SparseVolumeTextureMemory, FPlatformMemory::MCR_GPU, );

class UStreamableSparseVolumeTexture;
class USparseVolumeTextureFrame;

namespace UE
{
	namespace DerivedData
	{
		class FRequestOwner; // Can't include DDC headers from here, so we have to forward declare
		struct FCacheGetChunkRequest;
	}
}

namespace UE
{
namespace SVT
{

struct FResources;
struct FMipLevelStreamingInfo;
class FTextureRenderResources;
class FTileDataTexture;
class FTileUploader;
class FPageTableUpdater;
struct FStreamingInstanceRequest;
class FStreamingInstance;

// Helper class for managing slots in the tile data texture (SVT streaming pool). It uses a priority queue to reuse older slots that haven't been referenced for some (render) frames.
class FTileAllocator
{
public:
	static constexpr uint32 PhysicalCoordMask = (1u << 24u) - 1u; // Lower 24 bits are used for storing XYZ in 8 bit each. Upper 8 bit can be used by the caller. 

	// Describes a slot in the tile data texture and the current tile that is mapped to it.
	struct FAllocation
	{
		uint32 TileIndexInFrame;	// Index of the tile in the array of tiles of the SVT frame.
		uint16 FrameIndex;			// Index of the SVT frame this tile belongs to.
		uint16 bIsLocked : 1;		// Locked tiles are not in the priority queue and will never be streamed out/replaced unless manually freed.
		uint16 bIsAllocated : 1;	// Whether this slot is currently occupied by a tile.

		FAllocation() : TileIndexInFrame(0), FrameIndex(0), bIsLocked(0), bIsAllocated(0) {}
		FAllocation(uint16 InFrameIndex, uint32 InTileIndex, bool bInIsLocked, bool bInIsAllocated) 
			: TileIndexInFrame(InTileIndex), FrameIndex(InFrameIndex), bIsLocked(bInIsLocked), bIsAllocated(bInIsAllocated) {}
	};

	void Init(const FIntVector3& InResolutionInTiles);

	// Allocate a tile. Returns a packed 8|8|8 coordinate into the tile data texture or INDEX_NONE if there is no more space.
	// UpdateIndex is the index of the current tick/update.
	// FreeThreshold is an additional buffer to how many updates must have passed until a tile may be reused.
	// FrameIndex is the index of the SVT frame and TileIndexInFrame is the index of the tile for which we try to allocate a slot.
	// TilePriority is used to sort tiles in the priority queue. Higher values make the tile stream out after other tiles with lower values.
	// If bLocked is true, then the tile will never be automatically streamed out.
	// OutPreviousAllocation is a description of the old tile that occupied the newly allocated slot.
	uint32 Allocate(uint32 UpdateIndex, uint32 FreeThreshold, uint16 FrameIndex, uint32 TileIndexInFrame, uint32 TilePriority, bool bLocked, FAllocation& OutPreviousAllocation);
	
	// Marks a tile as still in use. See Allocate() for a description of the parameters.
	void UpdateUsage(uint32 UpdateIndex, uint32 TileCoord, uint32 TilePriority);
	
	// Frees a given tile. Expects the output of Allocate(), i.e. a packed 8|8|8 coordinate. Freed tiles are placed at the front of the priority queue for tile reuse.
	void Free(uint32 TileCoord);

	// Returns the size of the memory allocated for bookkeeping.
	SIZE_T GetAllocatedSize() const;

private:
	FBinaryHeap<uint64, uint32> FreeHeap;
	TArray<FAllocation> Allocations; // One entry for every slot in the tile data texture. Maps to the current SVT frame occupying that tile
	FIntVector3 ResolutionInTiles = FIntVector3::ZeroValue;
	uint32 TileCapacity = 0;
	uint32 NumAllocated = 0;
};

class FStreamingManager : public FRenderResource, public IStreamingManager
{
public:
	FStreamingManager();
	~FStreamingManager();

	//~ Begin FRenderResource Interface.
	virtual void InitRHI(FRHICommandListBase& RHICmdList) override;
	virtual void ReleaseRHI() override;
	//~ End FRenderResource Interface.

	//~ Begin IStreamingManager Interface.
	virtual void Add_GameThread(UStreamableSparseVolumeTexture* SparseVolumeTexture) override;
	virtual void Remove_GameThread(UStreamableSparseVolumeTexture* SparseVolumeTexture) override;
	virtual void Request_GameThread(UStreamableSparseVolumeTexture* SparseVolumeTexture, uint32 StreamingInstanceKey, float FrameRate, float FrameIndex, float MipLevel, EStreamingRequestFlags Flags) override;
	virtual void Update_GameThread() override;
	
	virtual void Request(UStreamableSparseVolumeTexture* SparseVolumeTexture, uint32 StreamingInstanceKey, float FrameRate, float FrameIndex, float MipLevel, EStreamingRequestFlags Flags) override;
	virtual void BeginAsyncUpdate(FRDGBuilder& GraphBuilder, bool bUseAsyncThread) override;
	virtual void EndAsyncUpdate(FRDGBuilder& GraphBuilder) override;
	virtual const FStreamingDebugInfo* GetStreamingDebugInfo(FRDGBuilder& GraphBuilder) const override;
	//~ End IStreamingManager Interface.

private:
	friend class FStreamingUpdateTask;

	// One per frame of each SVT
	struct FFrameInfo
	{
		const FResources* Resources;						// Initialized on the game thread
		FTextureRenderResources* TextureRenderResources;	// Initialized on the game thread

		int32 NumMipLevels = 0; // Number of actual mip levels of this frame. Depending on virtual volume extents, frames can have different numbers of levels.
		TArray<uint32> TileAllocations; // TileAllocations[PhysicalTileIndex]
		TBitArray<> ResidentPages; //  One bit for every (non-zero) page (in the sparse page octree) for all mip levels, starting at the highest mip
		TBitArray<> InvalidatedPages; // Temporarly needed when patching the page table. Marks all pages that require an update
		TBitArray<> ResidentTiles; // Tiles that are actually resident in GPU memory
		TBitArray<> StreamingTiles; // Tiles that have logically been streamed in but may not actually be resident in GPU memory yet because the IO request hasn't finished
		TMap<uint32, uint32> TileIndexToPendingRequestIndex;
		TRefCountPtr<IPooledRenderTarget> PageTableTexture;

		SIZE_T GetAllocatedSize() const;
	};

	// Streaming manager internal data for each SVT
	struct FStreamingInfo
	{
		uint16 SVTHandle = INDEX_NONE;
		FName SVTName = FName();
		EPixelFormat FormatA = PF_Unknown;
		EPixelFormat FormatB = PF_Unknown;
		FVector4f FallbackValueA = FVector4f();
		FVector4f FallbackValueB = FVector4f();
		int32 NumPrefetchFrames = 0;
		float PrefetchPercentageStepSize = 0.0f;
		float PrefetchPercentageBias = 0.0f;
		TArray<FFrameInfo> PerFrameInfo;
		TArray<TUniquePtr<FStreamingInstance>> StreamingInstances;
		TArray<uint32, TInlineAllocator<16>> MipLevelStreamingSize; // MipLevelStreamingSize[MipLevel] * FrameRate is the required IO bandwidth in bytes/s to stream a given MipLevel.

		FTileAllocator TileAllocator;
		TUniquePtr<FTileDataTexture> TileDataTexture;

		// Tries to look up an existing FStreamingInstance, creates a new one if it can't find one and finally updates it with the new request.
		FStreamingInstance* GetAndUpdateStreamingInstance(uint64 StreamingInstanceKey, const FStreamingInstanceRequest& Request);

		SIZE_T GetAllocatedSize() const;
	};

	// Represents an IO request for tile(s)
	struct FPendingRequest
	{
		uint16 SVTHandle = INDEX_NONE;
		uint16 FrameIndex = INDEX_NONE;
		uint32 TileOffset = INDEX_NONE;
		uint32 TileCount = 0;
#if WITH_EDITORONLY_DATA
		// When in editor, DDC requests finish on another thread, so we need to guard against race conditions.
		FCriticalSection DDCAsyncGuard;
		FSharedBuffer SharedBuffer;
		enum class EState
		{
			None,
			DDC_Pending,
			DDC_Ready,
			DDC_Failed,
			Memory,
			Disk,
		} State = EState::None;
		uint32 RetryCount = 0;
		uint32 RequestVersion = 0;
#endif
		FIoBuffer RequestBuffer;
		FBulkDataBatchRequest Request;
		uint32 IssuedInFrame = 0;
		bool bBlocking = false;

		bool IsValid() const { return SVTHandle != uint16(INDEX_NONE); }

		void Set(uint16 InSVTHandle, uint16 InFrameIndex, uint32 InTileOffset, uint32 InTileCount, uint32 InIssuedInFrame, bool bInBlocking)
		{
			SVTHandle = InSVTHandle;
			FrameIndex = InFrameIndex;
			TileOffset = InTileOffset;
			TileCount = InTileCount;
			IssuedInFrame = InIssuedInFrame;
			bBlocking = bInBlocking;
		}

		void Reset()
		{
			SVTHandle = INDEX_NONE;
			FrameIndex = INDEX_NONE;
			TileOffset = INDEX_NONE;
			TileCount = INDEX_NONE;
#if WITH_EDITORONLY_DATA
			SharedBuffer.Reset();
			State = EState::None;
			RetryCount = 0;
			++RequestVersion;
#endif
			if (Request.IsPending())
			{
				Request.Cancel();
				Request.Wait(); // Even after calling Cancel(), we still need to Wait() before we can touch the RequestBuffer.
			}
			Request.Reset();
			RequestBuffer = {};
			IssuedInFrame = 0;
			bBlocking = false;
		}
	};

	// Encapsulates all the src/dst pointers for memcpy-ing the streamed data into GPU memory.
	// See the comment on FMipLevelStreamingInfo for details about these pointers.
	struct FTileDataTask
	{
		TStaticArray<uint8*, 2> DstOccupancyBitsPtrs;
		TStaticArray<uint8*, 2> DstTileDataOffsetsPtrs;
		TStaticArray<uint8*, 2> DstTileDataPtrs;
		uint8* DstPhysicalTileCoordsPtr;
		TStaticArray<const uint8*, 2> SrcOccupancyBitsPtrs;
		TStaticArray<const uint8*, 2> SrcVoxelDataPtrs;
		TStaticArray<uint32, 2> VoxelDataSizes;
		TStaticArray<uint32, 2> VoxelDataBaseOffsets;
		uint32 PhysicalTileCoord;
	};

	struct FAsyncState
	{
		int32 NumReadyRequests = 0;
		bool bUpdateActive = false;
		bool bUpdateIsAsync = false;
	};

	// Stores info extracted from the UStreamableSparseVolumeTexture and its USparseVolumeTextureFrames so that we don't need to access these uobjects on the rendering thread
	struct FNewSparseVolumeTextureInfo
	{
		UStreamableSparseVolumeTexture* SVT = nullptr; // Do not dereference!
		FName SVTName = FName();
		EPixelFormat FormatA = PF_Unknown;
		EPixelFormat FormatB = PF_Unknown;
		FVector4f FallbackValueA = FVector4f(0.0f, 0.0f, 0.0f, 0.0f);
		FVector4f FallbackValueB = FVector4f(0.0f, 0.0f, 0.0f, 0.0f);
		int32 NumMipLevelsGlobal = 0;
		float StreamingPoolSizeFactor = 0.0f;
		int32 NumPrefetchFrames = 0;
		float PrefetchPercentageStepSize = 0.0f;
		float PrefetchPercentageBias = 0.0f;
		TArray<FFrameInfo> FrameInfo; // Only Resources and TextureRenderResources are initialized
	};

	// Uniquely identifies the mip level of a frame in a static or animated SparseVolumeTexture
	struct FFrameKey
	{
		uint16 SVTHandle = INDEX_NONE;
		uint16 FrameIndex = INDEX_NONE;

		friend FORCEINLINE uint32 GetTypeHash(const FFrameKey& Key) { return HashCombine(GetTypeHash(Key.SVTHandle), GetTypeHash(Key.FrameIndex)); }
		FORCEINLINE bool operator==(const FFrameKey& Other) const { return (SVTHandle == Other.SVTHandle) && (FrameIndex == Other.FrameIndex); }
		FORCEINLINE bool operator!=(const FFrameKey& Other) const { return !(*this == Other); }
		FORCEINLINE bool operator<(const FFrameKey& Other) const { return SVTHandle != Other.SVTHandle ? SVTHandle < Other.SVTHandle : FrameIndex < Other.FrameIndex; }
	};

	// The payload associated with a request for a given frame of a SVT
	struct FRequestPayload
	{
		FStreamingInstance* StreamingInstance = nullptr;
		uint16 MipLevelMask = 0; // Bitmask of requested mip levels
		float LowestMipFraction = 0.0f; // Setting this to a value between 0 and less than 1 signifies that only a certain percentage of pages in this mip level should be streamed
		TStaticArray<uint8, 16> Priorities = TStaticArray<uint8, 16>(InPlace, 0); // Priority for each mip level with a set bit in MipLevelMask
	};

	struct FStreamingRequest
	{
		FFrameKey Key;
		FRequestPayload Payload;
	};

	// Represents a contiguous range of tiles in a given frame of a SVT. This is the output of filtering FStreamingRequest and is used to generate FPendingRequests.
	struct FTileRange
	{
		uint16 SVTHandle = INDEX_NONE;
		uint16 FrameIndex = INDEX_NONE;
		uint32 TileOffset = INDEX_NONE;
		uint32 TileCount = 0;
		uint8 Priority = 0; // Higher value means higher priority
	};

	TMap<UStreamableSparseVolumeTexture*, uint16> SparseVolumeTextureToHandle; // Do not dereference the key! We just read the pointer itself.
	TSparseArray<TUniquePtr<FStreamingInfo>> StreamingInfo; 
	TMap<FFrameKey, FRequestPayload> RequestsHashTable;
	TArray<FPendingRequest> PendingRequests; // Do not access any members of an element before first calling LOCK_PENDING_REQUEST() on the element!
#if WITH_EDITORONLY_DATA
	TUniquePtr<UE::DerivedData::FRequestOwner> RequestOwner;
	TUniquePtr<UE::DerivedData::FRequestOwner> RequestOwnerBlocking;
#endif

	TUniquePtr<class FPageTableUpdater> PageTableUpdater;
	FGraphEventArray AsyncTaskEvents;
	FAsyncState AsyncState;
	int32 MaxPendingRequests = 0;
	int32 NumPendingRequests = 0;
	int32 NextPendingRequestIndex = 0;
	uint32 UpdateIndex = 1;
	FDateTime InitTime = FDateTime::Now(); // The precise time doesn't matter, we only use it as a basis to compute time as a double.
	int64 TotalRequestedBandwidth = 0;

	// Transient lifetime
	TSet<FStreamingInstance*> ActiveStreamingInstances; // Instances that have had a request in them since the last update
	TSet<FTileDataTexture*> TileDataTexturesToUpdate;
	TSet<FFrameInfo*> InvalidatedSVTFrames; // Set of SVT frames where tiles have been streamed in or out. Used in PatchPageTable().
	TArray<FTileRange> TileRangesToStream; // Output of FilterRequests(), consumed in IssueRequests().
	TArray<FTileDataTask> UploadTasks; // Accessed on the async thread
	TBitArray<> ResidentPagesNew; // Used as temporary memory in PatchPageTable()
	TBitArray<> ResidentPagesDiff; // Used as temporary memory in PatchPageTable()

	void AddInternal(FRDGBuilder& GraphBuilder, FNewSparseVolumeTextureInfo&& NewSVTInfo);
	void RemoveInternal(UStreamableSparseVolumeTexture* SparseVolumeTexture);
	void AddRequest(const FStreamingRequest& Request);
	void AsyncUpdate();
	void ComputeBandwidthLimit();
	void FilterRequests();
	void IssueRequests();
	int32 DetermineReadyRequests();
	void InstallReadyRequests();
	void PatchPageTable(FRDGBuilder& GraphBuilder); // Patches the page table to reflect streamed in/out pages and to ensure non-resident mip levels fall back to coarser mip level tile data
	FStreamingInfo* FindStreamingInfo(uint16 SparseVolumeTextureHandle); // Returns nullptr if the key can't be found
	FStreamingInfo* FindStreamingInfo(UStreamableSparseVolumeTexture* SparseVolumeTexture); // Returns nullptr if the key can't be found
	int32 AllocatePendingRequestIndex(); // Allocates a request in the PendingRequests array. Returns INDEX_NONE when out of slots.

#if WITH_EDITORONLY_DATA
	UE::DerivedData::FCacheGetChunkRequest BuildDDCRequest(const FResources& Resources, uint32 FirstTileIndex, uint32 NumTiles, uint32 PendingRequestIndex, int32 ChunkIndex);
	void RequestDDCData(TConstArrayView<UE::DerivedData::FCacheGetChunkRequest> DDCRequests, bool bBlocking);
#endif

	SIZE_T GetAllocatedSize() const;
};

}
}
