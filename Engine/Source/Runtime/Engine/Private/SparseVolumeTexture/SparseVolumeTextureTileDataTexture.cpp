// Copyright Epic Games, Inc. All Rights Reserved.

#include "SparseVolumeTextureTileDataTexture.h"
#include "SparseVolumeTextureUtility.h"
#include "SparseVolumeTextureStreamingManager.h" // LogSparseVolumeTextureStreamingManager
#include "RenderTargetPool.h"

DECLARE_MEMORY_STAT_POOL(TEXT("TileData GPU Memory"), STAT_SVTTileDataMemory, STATGROUP_SparseVolumeTextureMemory, FPlatformMemory::MCR_GPU);

static int32 GSVTStreamingReservedResources = 1;
static FAutoConsoleVariableRef CVarSVTStreamingReservedResources(
	TEXT("r.SparseVolumeTexture.Streaming.UseReservedResources"),
	GSVTStreamingReservedResources,
	TEXT("Allocate the SVT tile data texture (streaming pool) as a reserved/virtual texture, backed by N small physical memory allocations to reduce fragmentation. This lifts the 2GB resource size limit and also allows for better GPU memory management when allocating the texture."),
	ECVF_ReadOnly | ECVF_RenderThreadSafe
);

static int32 GSVTStreamingReservedResourcesMemoryLimit = -1;
static FAutoConsoleVariableRef CVarSVTStreamingReservedResourcesMemoryLimit(
	TEXT("r.SparseVolumeTexture.Streaming.ReservedResourcesMemoryLimit"),
	GSVTStreamingReservedResourcesMemoryLimit,
	TEXT("Memory limit in MiB on the maximum size of the streaming pool textures when using reserved resources. Without this limit it is theoretically possible to allocate enormous amounts of memory. Set to -1 to disable the limit. Default: -1"),
	ECVF_ReadOnly | ECVF_RenderThreadSafe
);

namespace UE
{
namespace SVT
{

bool FTileDataTexture::ShouldUseReservedResources()
{
	const bool bRHISupportsReservedVolumeTextures = GRHIGlobals.ReservedResources.SupportsVolumeTextures;
	const bool bCVarEnabled = GSVTStreamingReservedResources != 0;
	return bRHISupportsReservedVolumeTextures && bCVarEnabled;
}

int64 FTileDataTexture::GetMaxTileDataTextureResourceSize(int32 InVoxelMemSize)
{
	// When using reserved resources, we are no longer bound by the 2GB resource size limit, so we can set our own limit.
	int64 MaxStreamingPoolResourceSize = SVT::MaxResourceSize;
	if (ShouldUseReservedResources())
	{
		// First compute the maximum size based on the texture format and the maximum texture dimensions. This limit still applies even with reserved resources.
		constexpr int64 MaxNumVoxels = (int64)SVT::MaxVolumeTextureDim * (int64)SVT::MaxVolumeTextureDim * (int64)SVT::MaxVolumeTextureDim;
		MaxStreamingPoolResourceSize = MaxNumVoxels * (int64)InVoxelMemSize;

		if (GSVTStreamingReservedResourcesMemoryLimit > 0)
		{
			constexpr int64 OneMiB = 1024LL * 1024LL;
			const int64 ReservedResourcesMemoryLimit = FMath::Min((int64)GSVTStreamingReservedResourcesMemoryLimit, INT64_MAX / OneMiB) * OneMiB;
			MaxStreamingPoolResourceSize = FMath::Min(MaxStreamingPoolResourceSize, ReservedResourcesMemoryLimit);
		}
	}
	return MaxStreamingPoolResourceSize;
}

FIntVector3 FTileDataTexture::GetVolumeResolutionInTiles(int32 InNumRequiredTiles)
{
	int32 TileVolumeResolutionCube = 1;
	while (TileVolumeResolutionCube * TileVolumeResolutionCube * TileVolumeResolutionCube < InNumRequiredTiles)
	{
		TileVolumeResolutionCube++;				// We use a simple loop to compute the minimum resolution of a cube to store all the tile data
	}
	FIntVector3 TileDataVolumeResolution = FIntVector3(TileVolumeResolutionCube, TileVolumeResolutionCube, TileVolumeResolutionCube);

	// Trim volume to reclaim some space
	while ((TileDataVolumeResolution.X * TileDataVolumeResolution.Y * (TileDataVolumeResolution.Z - 1)) > InNumRequiredTiles)
	{
		TileDataVolumeResolution.Z--;
	}
	while ((TileDataVolumeResolution.X * (TileDataVolumeResolution.Y - 1) * TileDataVolumeResolution.Z) > InNumRequiredTiles)
	{
		TileDataVolumeResolution.Y--;
	}
	while (((TileDataVolumeResolution.X - 1) * TileDataVolumeResolution.Y * TileDataVolumeResolution.Z) > InNumRequiredTiles)
	{
		TileDataVolumeResolution.X--;
	}

	return TileDataVolumeResolution;
}

FIntVector3 FTileDataTexture::GetLargestPossibleVolumeResolutionInTiles(int32 InVoxelMemSize)
{
	const int64 MaxStreamingPoolResourceSize = GetMaxTileDataTextureResourceSize(InVoxelMemSize);
	const int64 TileMemSize = SVT::NumVoxelsPerPaddedTile * InVoxelMemSize;
	const int64 NumMaxTiles = MaxStreamingPoolResourceSize / TileMemSize;
	constexpr int32 MaxDimensionInTiles = SVT::MaxVolumeTextureDim / SPARSE_VOLUME_TILE_RES_PADDED;

	// Find a cube with a volume as close to NumMaxTiles as possible but not exceeding MaxDimensionInTiles.
	int32 TileVolumeResolutionCube = 1;
	while ((((TileVolumeResolutionCube + 1) * (TileVolumeResolutionCube + 1) * (TileVolumeResolutionCube + 1)) <= NumMaxTiles) && ((TileVolumeResolutionCube + 1) <= MaxDimensionInTiles))
	{
		++TileVolumeResolutionCube;
	}

	// Try to add to the sides to get closer to NumMaxTiles without exceeding MaxDimensionInTiles
	FIntVector3 ResolutionInTiles = FIntVector3(TileVolumeResolutionCube, TileVolumeResolutionCube, TileVolumeResolutionCube);
	if ((((ResolutionInTiles.X + 1) * ResolutionInTiles.Y * ResolutionInTiles.Z) <= NumMaxTiles) && ((ResolutionInTiles.X + 1) <= MaxDimensionInTiles))
	{
		++ResolutionInTiles.X;
	}
	if (((ResolutionInTiles.X * (ResolutionInTiles.Y + 1) * ResolutionInTiles.Z) <= NumMaxTiles) && ((ResolutionInTiles.Y + 1) <= MaxDimensionInTiles))
	{
		++ResolutionInTiles.Y;
	}
	if (((ResolutionInTiles.X * ResolutionInTiles.Y * (ResolutionInTiles.Z + 1)) <= NumMaxTiles) && ((ResolutionInTiles.Z + 1) <= MaxDimensionInTiles))
	{
		++ResolutionInTiles.Z;
	}

	const FIntVector3 ResolutionInVoxels = ResolutionInTiles * SPARSE_VOLUME_TILE_RES_PADDED;
	check(IsInBounds(ResolutionInVoxels, FIntVector3::ZeroValue, FIntVector3(SVT::MaxVolumeTextureDim + 1)));
	check(ResolutionInVoxels.X <= SVT::MaxVolumeTextureDim && ResolutionInVoxels.Y <= SVT::MaxVolumeTextureDim && ResolutionInVoxels.Z <= SVT::MaxVolumeTextureDim);
	check(((int64)ResolutionInVoxels.X * (int64)ResolutionInVoxels.Y * (int64)ResolutionInVoxels.Z * (int64)InVoxelMemSize) <= MaxStreamingPoolResourceSize);

	return ResolutionInTiles;
}

FTileDataTexture::FTileDataTexture(const FIntVector3& InResolutionInTiles, EPixelFormat InFormatA, EPixelFormat InFormatB, const FVector4f& InFallbackValueA, const FVector4f& InFallbackValueB)
	: TileUploader(MakeUnique<FTileUploader>()),
	ResolutionInTiles(InResolutionInTiles),
	PhysicalTilesCapacity(InResolutionInTiles.X* InResolutionInTiles.Y* InResolutionInTiles.Z),
	FormatA(InFormatA),
	FormatB(InFormatB),
	FallbackValueA(InFallbackValueA),
	FallbackValueB(InFallbackValueB)
{
	const int64 MaxFormatSize = FMath::Max(GPixelFormats[FormatA].BlockBytes, GPixelFormats[FormatB].BlockBytes);
	const FIntVector3 LargestPossibleResolutionInTiles = GetLargestPossibleVolumeResolutionInTiles(MaxFormatSize);
	const int32 LargestPossiblePhysicalTilesCapacity = LargestPossibleResolutionInTiles.X * LargestPossibleResolutionInTiles.Y * LargestPossibleResolutionInTiles.Z;

	// Ensure that the tile data texture(s) do not exceed the memory size and resolution limits.
	if (PhysicalTilesCapacity > LargestPossiblePhysicalTilesCapacity
		|| (ResolutionInTiles.X * SPARSE_VOLUME_TILE_RES_PADDED) > SVT::MaxVolumeTextureDim
		|| (ResolutionInTiles.Y * SPARSE_VOLUME_TILE_RES_PADDED) > SVT::MaxVolumeTextureDim
		|| (ResolutionInTiles.Z * SPARSE_VOLUME_TILE_RES_PADDED) > SVT::MaxVolumeTextureDim)
	{
		ResolutionInTiles = LargestPossibleResolutionInTiles;
		PhysicalTilesCapacity = LargestPossiblePhysicalTilesCapacity;

		UE_LOGF(LogSparseVolumeTextureStreamingManager, Warning, "Requested SparseVolumeTexture tile data texture resolution (in tiles) (%i, %i, %i) exceeds the resource size limit. Using the maximum value of (%i, %i. %i) instead.",
			InResolutionInTiles.X, InResolutionInTiles.Y, InResolutionInTiles.Z,
			LargestPossibleResolutionInTiles.X, LargestPossibleResolutionInTiles.Y, LargestPossibleResolutionInTiles.Z);
	}

	const FIntVector3 Resolution = ResolutionInTiles * SPARSE_VOLUME_TILE_RES_PADDED;
	check(Resolution.X <= SVT::MaxVolumeTextureDim && Resolution.Y <= SVT::MaxVolumeTextureDim && Resolution.Z <= SVT::MaxVolumeTextureDim);
	const int64 MaxTileDataTextureResourceSize = GetMaxTileDataTextureResourceSize(MaxFormatSize);
	check(((int64)Resolution.X * (int64)Resolution.Y * (int64)Resolution.Z * (int64)GPixelFormats[FormatA].BlockBytes) <= MaxTileDataTextureResourceSize);
	check(((int64)Resolution.X * (int64)Resolution.Y * (int64)Resolution.Z * (int64)GPixelFormats[FormatB].BlockBytes) <= MaxTileDataTextureResourceSize);
}

void FTileDataTexture::BeginReserveUpload()
{
	check(UploaderState == EUploaderState::Ready || UploaderState == EUploaderState::Reserved);
	UploaderState = EUploaderState::Reserving;
	NumReservedUploadTiles = 0;
	NumReservedUploadVoxelsA = 0;
	NumReservedUploadVoxelsB = 0;
}

void FTileDataTexture::ReserveUpload(int32 NumTiles, int32 NumVoxelsA, int32 NumVoxelsB)
{
	check(UploaderState == EUploaderState::Reserving);
	check((NumReservedUploadTiles + NumTiles) >= 0);
	check((NumReservedUploadVoxelsA + NumVoxelsA) >= 0);
	check((NumReservedUploadVoxelsB + NumVoxelsB) >= 0);
	NumReservedUploadTiles += NumTiles;
	NumReservedUploadVoxelsA += NumVoxelsA;
	NumReservedUploadVoxelsB += NumVoxelsB;
}

void FTileDataTexture::EndReserveUpload()
{
	check(UploaderState == EUploaderState::Reserving);
	UploaderState = EUploaderState::Reserved;
}

void FTileDataTexture::BeginUpload(FRDGBuilder& GraphBuilder)
{
	check(UploaderState == EUploaderState::Reserved);
	TileUploader->Init(GraphBuilder, NumReservedUploadTiles, NumReservedUploadVoxelsA, NumReservedUploadVoxelsB, FormatA, FormatB);
	UploaderState = EUploaderState::Uploading;
}

FTileUploader::FAddResult FTileDataTexture::AddUpload(int32 NumTiles, int32 NumVoxelsA, int32 NumVoxelsB)
{
	check(UploaderState == EUploaderState::Uploading);
	return TileUploader->Add_GetRef(NumTiles, NumVoxelsA, NumVoxelsB);
}

void FTileDataTexture::EndUpload(FRDGBuilder& GraphBuilder)
{
	check(UploaderState == EUploaderState::Uploading);
	TileUploader->ResourceUploadTo(GraphBuilder, TileDataTextureA, TileDataTextureB, FallbackValueA, FallbackValueB);
	UploaderState = EUploaderState::Ready;
}

void FTileDataTexture::InitRHI(FRHICommandListBase& RHICmdList)
{
	const FIntVector3 Resolution = ResolutionInTiles * SPARSE_VOLUME_TILE_RES_PADDED;
	const ETextureCreateFlags ReservedResourceFlags = ShouldUseReservedResources() ? (TexCreate_ReservedResource | TexCreate_ImmediateCommit) : TexCreate_None;
	const ETextureCreateFlags Flags = ReservedResourceFlags | TexCreate_ShaderResource | TexCreate_UAV | TexCreate_3DTiling | TexCreate_ReduceMemoryWithTilingMode;
	if (FormatA != PF_Unknown)
	{
		TileDataTextureA = GRenderTargetPool.FindFreeElement(RHICmdList, FRDGTextureDesc::Create3D(Resolution, FormatA, FClearValueBinding::Black, Flags), TEXT("SparseVolumeTexture.PhysicalTileDataA.RHITexture"));
#if STATS
		const int64 MemorySize = static_cast<int64>(RHICalcTexturePlatformSize(TileDataTextureA->GetRHI()->GetDesc()).Size);
		INC_MEMORY_STAT_BY(STAT_SVTTileDataMemory, MemorySize);
		INC_MEMORY_STAT_BY(STAT_SVTGPUMemoryTotal, MemorySize);
#endif
	}
	if (FormatB != PF_Unknown)
	{
		TileDataTextureB = GRenderTargetPool.FindFreeElement(RHICmdList, FRDGTextureDesc::Create3D(Resolution, FormatB, FClearValueBinding::Black, Flags), TEXT("SparseVolumeTexture.PhysicalTileDataB.RHITexture"));
#if STATS
		const int64 MemorySize = static_cast<int64>(RHICalcTexturePlatformSize(TileDataTextureB->GetRHI()->GetDesc()).Size);
		INC_MEMORY_STAT_BY(STAT_SVTTileDataMemory, MemorySize);
		INC_MEMORY_STAT_BY(STAT_SVTGPUMemoryTotal, MemorySize);
#endif
	}
}

void FTileDataTexture::ReleaseRHI()
{
#if STATS
	if (FormatA != PF_Unknown)
	{
		const int64 MemorySize = static_cast<int64>(RHICalcTexturePlatformSize(TileDataTextureA->GetRHI()->GetDesc()).Size);
		DEC_MEMORY_STAT_BY(STAT_SVTTileDataMemory, MemorySize);
		DEC_MEMORY_STAT_BY(STAT_SVTGPUMemoryTotal, MemorySize);
	}
	if (FormatB != PF_Unknown)
	{
		const int64 MemorySize = static_cast<int64>(RHICalcTexturePlatformSize(TileDataTextureB->GetRHI()->GetDesc()).Size);
		DEC_MEMORY_STAT_BY(STAT_SVTTileDataMemory, MemorySize);
		DEC_MEMORY_STAT_BY(STAT_SVTGPUMemoryTotal, MemorySize);
	}
#endif
}

}
}