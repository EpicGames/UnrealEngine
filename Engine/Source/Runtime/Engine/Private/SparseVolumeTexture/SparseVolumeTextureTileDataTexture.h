// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "RenderResource.h"
#include "SparseVolumeTextureUpload.h"

class FRHICommandListBase;
class FRDGBuilder;
enum EPixelFormat : uint8;

namespace UE
{
namespace SVT
{

class FTileUploader;

// Represents the physical tile data texture that serves as backing memory for the streamed in tiles. While this is treated as a single logical texture,
// it currently supports up to two actual RHI textures.
class FTileDataTexture : public FRenderResource
{
public:
	enum class EUploaderState
	{
		Ready, Reserving, Reserved, Uploading
	};

	static bool ShouldUseReservedResources();
	static int64 GetMaxTileDataTextureResourceSize(int32 InVoxelMemSize);
	static FIntVector3 GetVolumeResolutionInTiles(int32 InNumRequiredTiles);
	static FIntVector3 GetLargestPossibleVolumeResolutionInTiles(int32 InVoxelMemSize);

	// Constructor. May change the requested ResolutionInTiles (and resulting PhysicalTilesCapacity) if it exceeds hardware limits.
	explicit FTileDataTexture(const FIntVector3& ResolutionInTiles, EPixelFormat FormatA, EPixelFormat FormatB, const FVector4f& FallbackValueA, const FVector4f& FallbackValueB);

	EUploaderState GetUploaderState() const { return UploaderState; }
	int32 GetTileCapacity() const { return PhysicalTilesCapacity; }
	FIntVector3 GetResolutionInTiles() const { return ResolutionInTiles; }
	TRefCountPtr<IPooledRenderTarget> GetTileDataTextureA() { return TileDataTextureA; }
	TRefCountPtr<IPooledRenderTarget> GetTileDataTextureB() { return TileDataTextureB; }

	// Transitions from EUploaderState::Ready (or EUploaderState::Reserved) to EUploaderState::Reserving and allows callers to call ReserveUpload() afterwards. Resets number of reserved tiles/voxels.
	void BeginReserveUpload();
	// Reserves space in the upload buffer to hold the given number of tiles and voxels in addition to all tiles and voxels reserved with prior calls to this function. Must be in EUploaderState::Reserving.
	void ReserveUpload(int32 NumTiles, int32 NumVoxelsA, int32 NumVoxelsB);
	// Transitions from EUploaderState::Reserving to EUploaderState::Reserved.
	void EndReserveUpload();
	// Transitions from EUploaderState::Reserved to EUploaderState::Uploading and allows callers to call AddUpload() afterwards.
	void BeginUpload(FRDGBuilder& GraphBuilder);
	// Returns pointers/offsets into upload buffer memory for the caller to write data into. Must be in EUploaderState::Uploading.
	FTileUploader::FAddResult AddUpload(int32 NumTiles, int32 NumVoxelsA, int32 NumVoxelsB);
	// Actually uploads the data written to the upload buffer and transitions from EUploaderState::Uploading to EUploaderState::Ready.
	void EndUpload(FRDGBuilder& GraphBuilder);

	virtual void InitRHI(FRHICommandListBase& RHICmdList) override;
	virtual void ReleaseRHI() override;

private:
	TUniquePtr<FTileUploader> TileUploader;
	FIntVector3 ResolutionInTiles;
	int32 PhysicalTilesCapacity;
	EPixelFormat FormatA;
	EPixelFormat FormatB;
	FVector4f FallbackValueA;
	FVector4f FallbackValueB;
	TRefCountPtr<IPooledRenderTarget> TileDataTextureA;
	TRefCountPtr<IPooledRenderTarget> TileDataTextureB;
	EUploaderState UploaderState = EUploaderState::Ready;
	int32 NumReservedUploadTiles = 0;
	int32 NumReservedUploadVoxelsA = 0;
	int32 NumReservedUploadVoxelsB = 0;
};

}
}
