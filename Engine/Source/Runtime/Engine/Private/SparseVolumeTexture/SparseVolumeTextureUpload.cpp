// Copyright Epic Games, Inc. All Rights Reserved.

#include "SparseVolumeTextureUpload.h"
#include "SparseVolumeTextureUtility.h"
#include "RenderCore.h"
#include "RenderGraph.h"
#include "RenderGraphBuilder.h"
#include "RenderGraphUtils.h"
#include "GlobalShader.h"

static int32 GSVTStreamingAsyncCompute = 1;
static FAutoConsoleVariableRef CVarSVTStreamingAsyncCompute(
	TEXT("r.SparseVolumeTexture.Streaming.AsyncCompute"),
	GSVTStreamingAsyncCompute,
	TEXT("Schedule GPU work in async compute queue."),
	ECVF_RenderThreadSafe
);

class FSparseVolumeTextureUpdateFromSparseBufferCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FSparseVolumeTextureUpdateFromSparseBufferCS);
	SHADER_USE_PARAMETER_STRUCT(FSparseVolumeTextureUpdateFromSparseBufferCS, FGlobalShader)

	class FTextureUpdateMask : SHADER_PERMUTATION_SPARSE_INT("TEXTURE_UPDATE_MASK", 1, 2, 3);
	using FPermutationDomain = TShaderPermutationDomain<FTextureUpdateMask>;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture3D, DstPhysicalTileTextureA)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture3D, DstPhysicalTileTextureB)
		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<float4>, SrcPhysicalTileBufferA)
		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<float4>, SrcPhysicalTileBufferB)
		SHADER_PARAMETER_RDG_BUFFER_SRV(ByteAddressBuffer, OccupancyBitsBuffer)
		SHADER_PARAMETER_RDG_BUFFER_SRV(ByteAddressBuffer, TileDataOffsetsBuffer)
		SHADER_PARAMETER_RDG_BUFFER_SRV(ByteAddressBuffer, DstTileCoordsBuffer)
		SHADER_PARAMETER(FVector4f, FallbackValueA)
		SHADER_PARAMETER(FVector4f, FallbackValueB)
		SHADER_PARAMETER(uint32, TileIndexOffset)
		SHADER_PARAMETER(uint32, SrcVoxelDataOffsetA)
		SHADER_PARAMETER(uint32, SrcVoxelDataOffsetB)
		SHADER_PARAMETER(uint32, NumTilesToCopy)
		SHADER_PARAMETER(uint32, BufferTileStep)
		SHADER_PARAMETER(uint32, NumDispatchedGroups)
		SHADER_PARAMETER(uint32, PaddedTileSize)
	END_SHADER_PARAMETER_STRUCT()

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return true;
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("UPDATE_TILE_TEXTURE_FROM_SPARSE_BUFFER"), 1);
	}
};
IMPLEMENT_GLOBAL_SHADER(FSparseVolumeTextureUpdateFromSparseBufferCS, "/Engine/Private/SparseVolumeTexture/UpdateSparseVolumeTexture.usf", "SparseVolumeTextureUpdateFromSparseBufferCS", SF_Compute);

class FSparseVolumeTextureUpdatePageTableCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FSparseVolumeTextureUpdatePageTableCS);
	SHADER_USE_PARAMETER_STRUCT(FSparseVolumeTextureUpdatePageTableCS, FGlobalShader)

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture3D, PageTable)
		SHADER_PARAMETER_RDG_BUFFER_SRV(ByteAddressBuffer, PageTableUpdates)
		SHADER_PARAMETER(uint32, UpdateCoordOffset)
		SHADER_PARAMETER(uint32, UpdatePayloadOffset)
		SHADER_PARAMETER(uint32, NumUpdates)
	END_SHADER_PARAMETER_STRUCT()

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return true;
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("UPDATE_PAGE_TABLE"), 1);
	}
};
IMPLEMENT_GLOBAL_SHADER(FSparseVolumeTextureUpdatePageTableCS, "/Engine/Private/SparseVolumeTexture/UpdateSparseVolumeTexture.usf", "SparseVolumeTextureUpdatePageTableCS", SF_Compute);

namespace UE
{
namespace SVT
{

bool UseAsyncComputeForStreaming()
{
	// Disable async compute for streaming systems when MGPU is active, to work around GPU hangs
	const bool bAsyncCompute = GSupportsEfficientAsyncCompute && (GSVTStreamingAsyncCompute != 0) && (GNumExplicitGPUsForRendering == 1);
	return bAsyncCompute;
}

FTileUploader::FTileUploader()
{
	ResetState();
}

void FTileUploader::Init(FRDGBuilder& GraphBuilder, int32 InMaxNumTiles, int32 InMaxNumVoxelsA, int32 InMaxNumVoxelsB, EPixelFormat InFormatA, EPixelFormat InFormatB)
{
	check(InMaxNumTiles >= 0);
	check(InMaxNumVoxelsA >= 0);
	check(InMaxNumVoxelsB >= 0);
	check(InFormatA != PF_Unknown || InFormatB != PF_Unknown);
	ResetState();
	MaxNumTiles = InMaxNumTiles;
	MaxNumVoxelsA = InMaxNumVoxelsA;
	MaxNumVoxelsB = InMaxNumVoxelsB;
	FormatA = InFormatA;
	FormatB = InFormatB;
	FormatSizeA = GPixelFormats[FormatA].BlockBytes;
	FormatSizeB = GPixelFormats[FormatB].BlockBytes;
	
	// Create a new set of buffers if the old set is already queued into RDG.
	if (IsRegistered(GraphBuilder, DstTileCoordsUploadBuffer))
	{
		OccupancyBitsUploadBuffer = nullptr;
		TileDataOffsetsUploadBuffer = nullptr;
		DstTileCoordsUploadBuffer = nullptr;
		TileDataAUploadBuffer = nullptr;
		TileDataBUploadBuffer = nullptr;
	}
	
	const int32 NumTextures = (FormatA == PF_Unknown || FormatB == PF_Unknown) ? 1 : 2;
	
	if (MaxNumTiles > 0)
	{
		FRHICommandListBase& RHICmdList = GraphBuilder.RHICmdList;
	
		// Occupancy bits
		{
			const uint32 BufferSize = NumTextures * MaxNumTiles * SVT::NumOccupancyWordsPerPaddedTile * sizeof(uint32);
			FRDGBufferDesc BufferDesc = FRDGBufferDesc::CreateByteAddressUploadDesc(BufferSize);
			BufferDesc.Usage |= EBufferUsageFlags::Dynamic; // Skip the unneeded copy from upload to VRAM resource on d3d12 RHI
			AllocatePooledBuffer(BufferDesc, OccupancyBitsUploadBuffer, TEXT("SparseVolumeTexture.OccupancyBitsUploadBuffer"));
	
			OccupancyBitsAPtr = (uint8*)RHICmdList.LockBuffer(OccupancyBitsUploadBuffer->GetRHI(), 0, BufferSize, RLM_WriteOnly);
			OccupancyBitsBPtr = OccupancyBitsAPtr + (FormatA != PF_Unknown ? (MaxNumTiles * SVT::NumOccupancyWordsPerPaddedTile * sizeof(uint32)) : 0);
		}
		// Tile data offsets
		{
			const uint32 BufferSize = NumTextures * MaxNumTiles * sizeof(uint32);
			FRDGBufferDesc BufferDesc = FRDGBufferDesc::CreateByteAddressUploadDesc(BufferSize);
			BufferDesc.Usage |= EBufferUsageFlags::Dynamic; // Skip the unneeded copy from upload to VRAM resource on d3d12 RHI
			AllocatePooledBuffer(BufferDesc, TileDataOffsetsUploadBuffer, TEXT("SparseVolumeTexture.TileDataOffsetsUploadBuffer"));
	
			// Due to a limit on the maximum number of texels in a buffer SRV, we need to upload the data in smaller chunks. In order to figure out the chunk offsets/sizes,
			// we need to read the TileDataOffset values the caller has written to the returned pointers. We want to avoid reading from a mapped upload buffer pointer,
			// which is why we use a temporary allocation to write the upload data to.
			TileDataOffsets.SetNumUninitialized(NumTextures * MaxNumTiles);
			TileDataOffsetsAPtr = (uint8*)TileDataOffsets.GetData();
			TileDataOffsetsBPtr = TileDataOffsetsAPtr + (FormatA != PF_Unknown ? (MaxNumTiles * sizeof(uint32)) : 0);
		}
		// TileCoords
		{
			const uint32 BufferSize = MaxNumTiles * sizeof(uint32);
			FRDGBufferDesc BufferDesc = FRDGBufferDesc::CreateByteAddressUploadDesc(BufferSize);
			BufferDesc.Usage |= EBufferUsageFlags::Dynamic; // Skip the unneeded copy from upload to VRAM resource on d3d12 RHI
			AllocatePooledBuffer(BufferDesc, DstTileCoordsUploadBuffer, TEXT("SparseVolumeTexture.TileCoordsUploadBuffer"));
	
			TileCoordsPtr = (uint8*)RHICmdList.LockBuffer(DstTileCoordsUploadBuffer->GetRHI(), 0, BufferSize, RLM_WriteOnly);
		}
	
		// TileData
		if (FormatSizeA > 0)
		{
			FRDGBufferDesc BufferDesc = FRDGBufferDesc::CreateUploadDesc(FormatSizeA, FMath::Max(MaxNumVoxelsA, 1));
			BufferDesc.Usage |= EBufferUsageFlags::Dynamic; // Skip the unneeded copy from upload to VRAM resource on d3d12 RHI
			AllocatePooledBuffer(BufferDesc, TileDataAUploadBuffer, TEXT("SparseVolumeTexture.TileDataAUploadBuffer"));
	
			TileDataAPtr = (uint8*)RHICmdList.LockBuffer(TileDataAUploadBuffer->GetRHI(), 0, FMath::Max(MaxNumVoxelsA, 1) * FormatSizeA, RLM_WriteOnly);
		}
		if (FormatSizeB > 0)
		{
			FRDGBufferDesc BufferDesc = FRDGBufferDesc::CreateUploadDesc(FormatSizeB, FMath::Max(MaxNumVoxelsB, 1));
			BufferDesc.Usage |= EBufferUsageFlags::Dynamic; // Skip the unneeded copy from upload to VRAM resource on d3d12 RHI
			AllocatePooledBuffer(BufferDesc, TileDataBUploadBuffer, TEXT("SparseVolumeTexture.TileDataBUploadBuffer"));
	
			TileDataBPtr = (uint8*)RHICmdList.LockBuffer(TileDataBUploadBuffer->GetRHI(), 0, FMath::Max(MaxNumVoxelsB, 1) * FormatSizeB, RLM_WriteOnly);
		}
	}
}

FTileUploader::FAddResult FTileUploader::Add_GetRef(int32 InNumTiles, int32 InNumVoxelsA, int32 InNumVoxelsB)
{
	check((NumWrittenTiles + InNumTiles) <= MaxNumTiles);
	check((NumWrittenVoxelsA + InNumVoxelsA) <= MaxNumVoxelsA);
	check((NumWrittenVoxelsB + InNumVoxelsB) <= MaxNumVoxelsB);
	check(TileCoordsPtr);
	check(FormatSizeA <= 0 || TileDataAPtr);
	check(FormatSizeB <= 0 || TileDataBPtr);
	
	FAddResult Result = {};
	Result.OccupancyBitsPtrs[0] = TileDataAPtr ? OccupancyBitsAPtr + (NumWrittenTiles * SVT::NumOccupancyWordsPerPaddedTile * sizeof(uint32)) : nullptr;
	Result.OccupancyBitsPtrs[1] = TileDataBPtr ? OccupancyBitsBPtr + (NumWrittenTiles * SVT::NumOccupancyWordsPerPaddedTile * sizeof(uint32)) : nullptr;
	Result.TileDataOffsetsPtrs[0] = TileDataAPtr ? TileDataOffsetsAPtr + (NumWrittenTiles * sizeof(uint32)) : nullptr;
	Result.TileDataOffsetsPtrs[1] = TileDataBPtr ? TileDataOffsetsBPtr + (NumWrittenTiles * sizeof(uint32)) : nullptr;
	Result.TileDataPtrs[0] = TileDataAPtr ? TileDataAPtr + (NumWrittenVoxelsA * FormatSizeA) : nullptr;
	Result.TileDataPtrs[1] = TileDataBPtr ? TileDataBPtr + (NumWrittenVoxelsB * FormatSizeB) : nullptr;
	Result.TileDataBaseOffsets[0] = NumWrittenVoxelsA;
	Result.TileDataBaseOffsets[1] = NumWrittenVoxelsB;
	Result.PackedPhysicalTileCoordsPtr = TileCoordsPtr + NumWrittenTiles * sizeof(uint32);
	
	NumWrittenTiles += InNumTiles;
	NumWrittenVoxelsA += InNumVoxelsA;
	NumWrittenVoxelsB += InNumVoxelsB;
	
	return Result;
}

void FTileUploader::Release()
{
	OccupancyBitsUploadBuffer.SafeRelease();
	TileDataOffsetsUploadBuffer.SafeRelease();
	DstTileCoordsUploadBuffer.SafeRelease();
	TileDataAUploadBuffer.SafeRelease();
	TileDataBUploadBuffer.SafeRelease();
	TileDataOffsets.Reset();
	ResetState();
}

void FTileUploader::ResourceUploadTo(FRDGBuilder& GraphBuilder, const TRefCountPtr<IPooledRenderTarget>& InDstTextureA, const TRefCountPtr<IPooledRenderTarget>& InDstTextureB, const FVector4f& InFallbackValueA, const FVector4f& InFallbackValueB)
{
	check(InDstTextureA || FormatSizeA <= 0);
	check(InDstTextureB || FormatSizeB <= 0);
	if (MaxNumTiles > 0)
	{
		FRHICommandListBase& RHICmdList = GraphBuilder.RHICmdList;
	
		RHICmdList.UnlockBuffer(OccupancyBitsUploadBuffer->GetRHI());
		RHICmdList.UnlockBuffer(DstTileCoordsUploadBuffer->GetRHI());
	
		// TileDataOffset values were written to a temporary allocation so that we can access them later in this function. Unlike the other buffers, we now need to copy that data over to the actual upload buffer.
		void* TileDataOffsetsUploadPtr = RHICmdList.LockBuffer(TileDataOffsetsUploadBuffer->GetRHI(), 0, TileDataOffsets.Num() * sizeof(TileDataOffsets[0]), RLM_WriteOnly);
		FMemory::Memcpy(TileDataOffsetsUploadPtr, TileDataOffsets.GetData(), TileDataOffsets.Num() * sizeof(TileDataOffsets[0]));
		RHICmdList.UnlockBuffer(TileDataOffsetsUploadBuffer->GetRHI());
	
		if (TileDataAPtr)
		{
			RHICmdList.UnlockBuffer(TileDataAUploadBuffer->GetRHI());
		}
		if (TileDataBPtr)
		{
			RHICmdList.UnlockBuffer(TileDataBUploadBuffer->GetRHI());
		}
	
		if (NumWrittenTiles > 0)
		{
			FRDGTexture* DstTextureARDG = nullptr;
			FRDGTexture* DstTextureBRDG = nullptr;
			if (InDstTextureA)
			{
				DstTextureARDG = GraphBuilder.RegisterExternalTexture(InDstTextureA);
				GraphBuilder.UseInternalAccessMode(DstTextureARDG);
			}
			if (InDstTextureB)
			{
				DstTextureBRDG = GraphBuilder.RegisterExternalTexture(InDstTextureB);
				GraphBuilder.UseInternalAccessMode(DstTextureBRDG);
			}

			FRDGBuffer* SrcBufferARDG = FormatSizeA > 0 ? GraphBuilder.RegisterExternalBuffer(TileDataAUploadBuffer) : nullptr;
			FRDGBuffer* SrcBufferBRDG = FormatSizeB > 0 ? GraphBuilder.RegisterExternalBuffer(TileDataBUploadBuffer) : nullptr;
			check(SrcBufferARDG || SrcBufferBRDG);
	
			FRDGTextureUAV* DstTextureAUAV = DstTextureARDG ? GraphBuilder.CreateUAV(DstTextureARDG) : nullptr;
			FRDGTextureUAV* DstTextureBUAV = DstTextureBRDG ? GraphBuilder.CreateUAV(DstTextureBRDG) : nullptr;
	
			FRDGBufferSRV* OccupancyBitsBufferSRV = GraphBuilder.CreateSRV(GraphBuilder.RegisterExternalBuffer(OccupancyBitsUploadBuffer));
			FRDGBufferSRV* TileDataOffsetsBufferSRV = GraphBuilder.CreateSRV(GraphBuilder.RegisterExternalBuffer(TileDataOffsetsUploadBuffer));
			FRDGBufferSRV* DstTileCoordsBufferSRV = GraphBuilder.CreateSRV(GraphBuilder.RegisterExternalBuffer(DstTileCoordsUploadBuffer));
			// Either SrcBufferARDG or SrcBufferBRDG must exist and will have at least 1 element.
			FRDGBufferSRV* DummySrcBufferSRV = SrcBufferARDG ? GraphBuilder.CreateSRV(SrcBufferARDG, FormatA) : GraphBuilder.CreateSRV(SrcBufferBRDG, FormatB);
	
			uint32 TextureUpdateMask = 0;
			TextureUpdateMask |= FormatSizeA > 0 ? 0x1u : 0x0u;
			TextureUpdateMask |= FormatSizeB > 0 ? 0x2u : 0x0u;
			check(TextureUpdateMask != 0);

			FSparseVolumeTextureUpdateFromSparseBufferCS::FPermutationDomain CSPermutationDomain;
			CSPermutationDomain.Set<FSparseVolumeTextureUpdateFromSparseBufferCS::FTextureUpdateMask>(TextureUpdateMask);
			auto ComputeShader = GetGlobalShaderMap(GMaxRHIFeatureLevel)->GetShader<FSparseVolumeTextureUpdateFromSparseBufferCS>(CSPermutationDomain);
	
			int32 NumUploadedTiles = 0;
			int32 NumUploadedVoxelsA = 0;
			int32 NumUploadedVoxelsB = 0;
	
			// This is a limit on some platforms on the maximum number of texels in a texel/typed buffer. Unfortunately for R8 (1 byte) formats, this means that we can upload only 1/16 of all the texels of a 2GB texture.
			// In order to work around this issue, the data to be uploaded is split into chunks such that this limit is not violated. We can use the TileDataOffsets values to get the number of voxels per tile and then
			// just add as many tiles to the batch as we can.
			const int32 MaxNumTexelsPerResource = 1 << 27;
	
			while (NumUploadedTiles < NumWrittenTiles)
			{
				// Determine the number of tiles to upload in this iteration
				int32 NumTilesInThisBatch = 0;
				int32 NumVoxelsAInThisBatch = 0;
				int32 NumVoxelsBInThisBatch = 0;
				for (int32 TileIndex = NumUploadedTiles; TileIndex < NumWrittenTiles; ++TileIndex)
				{
					const int32 VoxelOffsetA = FormatSizeA > 0 ? reinterpret_cast<uint32*>(TileDataOffsetsAPtr)[TileIndex] : 0;
					const int32 VoxelOffsetB = FormatSizeB > 0 ? reinterpret_cast<uint32*>(TileDataOffsetsBPtr)[TileIndex] : 0;
					const int32 VoxelEndIndexA = FormatSizeA > 0 && (TileIndex + 1) < NumWrittenTiles ? reinterpret_cast<uint32*>(TileDataOffsetsAPtr)[TileIndex + 1] : NumWrittenVoxelsA;
					const int32 VoxelEndIndexB = FormatSizeB > 0 && (TileIndex + 1) < NumWrittenTiles ? reinterpret_cast<uint32*>(TileDataOffsetsBPtr)[TileIndex + 1] : NumWrittenVoxelsB;
					const int32 TileNumVoxelsA = VoxelEndIndexA - VoxelOffsetA;
					const int32 TileNumVoxelsB = VoxelEndIndexB - VoxelOffsetB;
					check(TileNumVoxelsA >= 0 && TileNumVoxelsA <= SVT::NumVoxelsPerPaddedTile);
					check(TileNumVoxelsB >= 0 && TileNumVoxelsB <= SVT::NumVoxelsPerPaddedTile);
	
					// Adding additional voxels to the batch would exceed the limit, so exit the loop and upload the data.
					if ((NumVoxelsAInThisBatch + TileNumVoxelsA) > MaxNumTexelsPerResource || (NumVoxelsBInThisBatch + TileNumVoxelsB) > MaxNumTexelsPerResource)
					{
						break;
					}
	
					NumTilesInThisBatch += 1;
					NumVoxelsAInThisBatch += TileNumVoxelsA;
					NumVoxelsBInThisBatch += TileNumVoxelsB;
				}
	
				FRDGBufferSRV* TileDataABufferSRV = nullptr;
				FRDGBufferSRV* TileDataBBufferSRV = nullptr;
	
				// This is the critical part: For every batch we create a SRV scoped to a range within the voxel data upload buffer still fitting within the typed buffer texel limit.
				if (FormatSizeA && NumVoxelsAInThisBatch)
				{
					FRDGBufferSRVDesc SRVDesc(SrcBufferARDG, FormatA);
					SRVDesc.StartOffsetBytes = NumUploadedVoxelsA * FormatSizeA;
					SRVDesc.NumElements = NumVoxelsAInThisBatch;
					TileDataABufferSRV = GraphBuilder.CreateSRV(SRVDesc);
				}
				if (FormatSizeB && NumVoxelsBInThisBatch)
				{
					FRDGBufferSRVDesc SRVDesc(SrcBufferBRDG, FormatB);
					SRVDesc.StartOffsetBytes = NumUploadedVoxelsB * FormatSizeB;
					SRVDesc.NumElements = NumVoxelsBInThisBatch;
					TileDataBBufferSRV = GraphBuilder.CreateSRV(SRVDesc);
				}
	
				FSparseVolumeTextureUpdateFromSparseBufferCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FSparseVolumeTextureUpdateFromSparseBufferCS::FParameters>();
				PassParameters->DstPhysicalTileTextureA = DstTextureAUAV;
				PassParameters->DstPhysicalTileTextureB = DstTextureBUAV;
				PassParameters->SrcPhysicalTileBufferA = TileDataABufferSRV ? TileDataABufferSRV : DummySrcBufferSRV; // TileDataABufferSRV and TileDataBBufferSRV can be null if there are no explicitly stored voxels to upload
				PassParameters->SrcPhysicalTileBufferB = TileDataBBufferSRV ? TileDataBBufferSRV : DummySrcBufferSRV;
				PassParameters->OccupancyBitsBuffer = OccupancyBitsBufferSRV;
				PassParameters->TileDataOffsetsBuffer = TileDataOffsetsBufferSRV;
				PassParameters->DstTileCoordsBuffer = DstTileCoordsBufferSRV;
				PassParameters->FallbackValueA = InFallbackValueA;
				PassParameters->FallbackValueB = InFallbackValueB;
				PassParameters->TileIndexOffset = NumUploadedTiles; // This lets the shader know how many tiles have already been processed in previous dispatches.
				PassParameters->SrcVoxelDataOffsetA = NumUploadedVoxelsA; // SrcVoxelDataOffsetA and SrcVoxelDataOffsetB are subtracted from the calculated voxel data buffer read indices
				PassParameters->SrcVoxelDataOffsetB = NumUploadedVoxelsB;
				PassParameters->NumTilesToCopy = NumTilesInThisBatch;
				PassParameters->BufferTileStep = MaxNumTiles;
				PassParameters->NumDispatchedGroups = FMath::Min(NumTilesInThisBatch, GRHIMaxDispatchThreadGroupsPerDimension.X);
				PassParameters->PaddedTileSize = SPARSE_VOLUME_TILE_RES_PADDED;
	
				const bool bAsyncCompute = UseAsyncComputeForStreaming();
	
				FComputeShaderUtils::AddPass(
					GraphBuilder,
					RDG_EVENT_NAME("Upload SVT Tiles (TileCount: %u)", NumTilesInThisBatch),
					bAsyncCompute ? ERDGPassFlags::AsyncCompute : ERDGPassFlags::Compute,
					ComputeShader,
					PassParameters,
					FIntVector3(PassParameters->NumDispatchedGroups, 1, 1)
				);
	
				NumUploadedTiles += NumTilesInThisBatch;
				NumUploadedVoxelsA += NumVoxelsAInThisBatch;
				NumUploadedVoxelsB += NumVoxelsBInThisBatch;
			}
	
			check(NumUploadedTiles == NumWrittenTiles);
			check(NumUploadedVoxelsA == NumWrittenVoxelsA);
			check(NumUploadedVoxelsB == NumWrittenVoxelsB);

			// Transition back to SRV in case we want to access them using their raw RHI resources within this graph
			if (DstTextureARDG)
			{
				GraphBuilder.UseExternalAccessMode(DstTextureARDG, ERHIAccess::SRVMask, ERHIPipeline::All);
			}
			if (DstTextureBRDG)
			{
				GraphBuilder.UseExternalAccessMode(DstTextureBRDG, ERHIAccess::SRVMask, ERHIPipeline::All);
			}
		}
	}
	Release();
}

void FTileUploader::ResetState()
{
	OccupancyBitsAPtr = nullptr;
	OccupancyBitsBPtr = nullptr;
	TileDataOffsetsAPtr = nullptr;
	TileDataOffsetsBPtr = nullptr;
	TileCoordsPtr = nullptr;
	TileDataAPtr = nullptr;
	TileDataBPtr = nullptr;
	MaxNumTiles = 0;
	MaxNumVoxelsA = 0;
	MaxNumVoxelsB = 0;
	FormatA = PF_Unknown;
	FormatB = PF_Unknown;
	FormatSizeA = 0;
	FormatSizeB = 0;
	NumWrittenTiles = 0;
	NumWrittenVoxelsA = 0;
	NumWrittenVoxelsB = 0;
}

FPageTableUpdater::FPageTableUpdater()
{
	ResetState();
}

void FPageTableUpdater::Init(FRDGBuilder& GraphBuilder, int32 InMaxNumUpdates, int32 InEstimatedNumBatches)
{
	ResetState();
	MaxNumUpdates = InMaxNumUpdates;
	Batches.Reserve(InEstimatedNumBatches);
	
	// Create a new buffer if the old one is already queued into RDG.
	if (IsRegistered(GraphBuilder, UpdatesUploadBuffer))
	{
		UpdatesUploadBuffer = nullptr;
	}
	
	if (MaxNumUpdates > 0)
	{
		// Add EBufferUsageFlags::Dynamic to skip the unneeded copy from upload to VRAM resource on d3d12 RHI
		FRDGBufferDesc BufferDesc = FRDGBufferDesc::CreateByteAddressUploadDesc(MaxNumUpdates * 2 * sizeof(uint32));
		BufferDesc.Usage |= EBufferUsageFlags::Dynamic;
		AllocatePooledBuffer(BufferDesc, UpdatesUploadBuffer, TEXT("SparseVolumeTexture.PageTableUpdatesUploadBuffer"));
	
		DataPtr = (uint8*)GraphBuilder.RHICmdList.LockBuffer(UpdatesUploadBuffer->GetRHI(), 0, MaxNumUpdates * 2 * sizeof(uint32), RLM_WriteOnly);
	}
}

void FPageTableUpdater::Add_GetRef(const TRefCountPtr<IPooledRenderTarget>& InPageTable, int32 InMipLevel, int32 InNumUpdates, uint8*& OutCoordsPtr, uint8*& OutPayloadPtr)
{
	check((NumWrittenUpdates + InNumUpdates) <= MaxNumUpdates);
	check(DataPtr);
	FBatch* Batch = Batches.IsEmpty() ? nullptr : &Batches.Last();
	if (!Batch || Batch->PageTable != InPageTable || Batch->MipLevel != InMipLevel)
	{
		Batch = &Batches.Add_GetRef(FBatch(InPageTable, InMipLevel));
	}
	
	OutCoordsPtr = DataPtr + NumWrittenUpdates * sizeof(uint32);
	OutPayloadPtr = DataPtr + (MaxNumUpdates + NumWrittenUpdates) * sizeof(uint32);
	
	Batch->NumUpdates += InNumUpdates;
	NumWrittenUpdates += InNumUpdates;
}

void FPageTableUpdater::Release()
{
	UpdatesUploadBuffer.SafeRelease();
	ResetState();
}

void FPageTableUpdater::Apply(FRDGBuilder& GraphBuilder)
{
	if (MaxNumUpdates > 0)
	{
		GraphBuilder.RHICmdList.UnlockBuffer(UpdatesUploadBuffer->GetRHI());
	
		if (NumWrittenUpdates > 0)
		{
			const bool bAsyncCompute = UseAsyncComputeForStreaming();
			auto ComputeShader = GetGlobalShaderMap(GMaxRHIFeatureLevel)->GetShader<FSparseVolumeTextureUpdatePageTableCS>();

			// Register page table textures with RDG and set it to internal access mode
			for (const FBatch& Batch : Batches)
			{
				FRDGTexture* PageTableRDG = GraphBuilder.FindExternalTexture(Batch.PageTable);
				if (!PageTableRDG)
				{
					PageTableRDG = GraphBuilder.RegisterExternalTexture(Batch.PageTable);
				}
				GraphBuilder.UseInternalAccessMode(PageTableRDG); // Make sure to use graph tracking in case we previously called UseExternalAccessMode() within the current graph
			}
	
			uint32 UpdatesOffset = 0;
			for (const FBatch& Batch : Batches)
			{
				FRDGTexture* PageTableRDG = GraphBuilder.FindExternalTexture(Batch.PageTable);
				check(PageTableRDG);
				FRDGTextureUAV* PageTableUAV = GraphBuilder.CreateUAV(FRDGTextureUAVDesc(PageTableRDG, Batch.MipLevel, PF_R32_UINT));
				FRDGBufferSRV* UpdatesBufferSRV = GraphBuilder.CreateSRV(GraphBuilder.RegisterExternalBuffer(UpdatesUploadBuffer));
	
				FSparseVolumeTextureUpdatePageTableCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FSparseVolumeTextureUpdatePageTableCS::FParameters>();
				PassParameters->PageTable = PageTableUAV;
				PassParameters->PageTableUpdates = UpdatesBufferSRV;
				PassParameters->UpdateCoordOffset = UpdatesOffset;
				PassParameters->UpdatePayloadOffset = MaxNumUpdates + UpdatesOffset;
				PassParameters->NumUpdates = Batch.NumUpdates;
	
				FComputeShaderUtils::AddPass(
					GraphBuilder,
					RDG_EVENT_NAME("Update SVT PageTable (UpdateCount: %u)", Batch.NumUpdates),
					bAsyncCompute ? ERDGPassFlags::AsyncCompute : ERDGPassFlags::Compute,
					ComputeShader,
					PassParameters,
					FComputeShaderUtils::GetGroupCount(Batch.NumUpdates, 64)
				);
	
				UpdatesOffset += Batch.NumUpdates;
			}

			// Transition back to SRV in case we want to access the page table texture using its raw RHI resource within this graph
			for (const FBatch& Batch : Batches)
			{
				FRDGTexture* PageTableRDG = GraphBuilder.FindExternalTexture(Batch.PageTable);
				check(PageTableRDG);
				GraphBuilder.UseExternalAccessMode(PageTableRDG, ERHIAccess::SRVMask, ERHIPipeline::All);
			}
		}
	}
	
	Release();
}

void FPageTableUpdater::ResetState()
{
	Batches.Reset();
	DataPtr = nullptr;
	NumWrittenUpdates = 0;
	MaxNumUpdates = 0;
}

}
}