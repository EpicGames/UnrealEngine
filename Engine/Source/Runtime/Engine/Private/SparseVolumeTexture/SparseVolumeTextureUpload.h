// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "CoreMinimal.h"
#include "Containers/StaticArray.h"
#include "RenderGraphResources.h"

class FRDGBuilder;
class FRDGPooledBuffer;
class FRHITexture;
enum EPixelFormat : uint8;

namespace UE
{
namespace SVT
{

// Whether to use async compute for all SVT streaming GPU work
bool UseAsyncComputeForStreaming();

// Utility class for uploading tiles to a physical tile data texture
class FTileUploader
{
public:

	struct FAddResult
	{
		TStaticArray<uint8*, 2> OccupancyBitsPtrs;
		TStaticArray<uint8*, 2> TileDataOffsetsPtrs;
		TStaticArray<uint8*, 2> TileDataPtrs;
		TStaticArray<uint32, 2> TileDataBaseOffsets; // Caller needs to add this value to all data written to TileDataOffsetsPtrs
		uint8* PackedPhysicalTileCoordsPtr;
	};

	explicit FTileUploader();
	void Init(FRDGBuilder& InGraphBuilder, int32 InMaxNumTiles, int32 InMaxNumVoxelsA, int32 InMaxNumVoxelsB, EPixelFormat InFormatA, EPixelFormat InFormatB);
	FAddResult Add_GetRef(int32 InNumTiles, int32 InNumVoxelsA, int32 InNumVoxelsB);
	void Release();
	void ResourceUploadTo(FRDGBuilder& InGraphBuilder, const TRefCountPtr<IPooledRenderTarget>& InDstTextureA, const TRefCountPtr<IPooledRenderTarget>& InDstTextureB, const FVector4f& InFallbackValueA, const FVector4f& InFallbackValueB);

private:
	TRefCountPtr<FRDGPooledBuffer> OccupancyBitsUploadBuffer;
	TRefCountPtr<FRDGPooledBuffer> TileDataOffsetsUploadBuffer;
	TRefCountPtr<FRDGPooledBuffer> DstTileCoordsUploadBuffer;
	TRefCountPtr<FRDGPooledBuffer> TileDataAUploadBuffer;
	TRefCountPtr<FRDGPooledBuffer> TileDataBUploadBuffer;
	TArray<uint32> TileDataOffsets; // CPU-readable per-tile offsets into tile data
	uint8* OccupancyBitsAPtr;
	uint8* OccupancyBitsBPtr;
	uint8* TileDataOffsetsAPtr;
	uint8* TileDataOffsetsBPtr;
	uint8* TileCoordsPtr;
	uint8* TileDataAPtr;
	uint8* TileDataBPtr;
	int32 MaxNumTiles;
	int32 MaxNumVoxelsA;
	int32 MaxNumVoxelsB;
	EPixelFormat FormatA;
	EPixelFormat FormatB;
	int32 FormatSizeA;
	int32 FormatSizeB;
	int32 NumWrittenTiles;
	int32 NumWrittenVoxelsA;
	int32 NumWrittenVoxelsB;

	void ResetState();
};

// Utility class for writing page table entries
class FPageTableUpdater
{
public:
	explicit FPageTableUpdater();
	void Init(FRDGBuilder& InGraphBuilder, int32 InMaxNumUpdates, int32 InEstimatedNumBatches);
	void Add_GetRef(const TRefCountPtr<IPooledRenderTarget>& InPageTable, int32 InMipLevel, int32 InNumUpdates, uint8*& OutCoordsPtr, uint8*& OutPayloadPtr);
	void Release();
	void Apply(FRDGBuilder& InGraphBuilder);

private:
	struct FBatch
	{
		TRefCountPtr<IPooledRenderTarget> PageTable;
		int32 MipLevel;
		int32 NumUpdates;

		FBatch() = default;
		FBatch(const TRefCountPtr<IPooledRenderTarget>& InPageTable, int32 InMipLevel) : PageTable(InPageTable), MipLevel(InMipLevel), NumUpdates(0) {}
	};

	TRefCountPtr<FRDGPooledBuffer> UpdatesUploadBuffer;
	TArray<FBatch> Batches;
	uint8* DataPtr;
	int32 NumWrittenUpdates;
	int32 MaxNumUpdates;

	void ResetState();
};

}
}