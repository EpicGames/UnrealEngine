// Copyright Epic Games, Inc. All Rights Reserved.

#include "SparseVolumeTextureUtility.h"

namespace UE
{
namespace SVT
{
namespace Private
{
	uint16 F32ToF16(float Value)
	{
		FFloat16 F16(Value);
		return F16.Encoded;
	};

	float F16ToF32(uint16 Packed)
	{
		FFloat16 F16{};
		F16.Encoded = Packed;
		return F16.GetFloat();
	};
} // Private
} // SVT
} // UE

uint32 UE::SVT::PackX11Y11Z10(const FIntVector3& Value)
{
	return (Value.X & 0x7FFu) | ((Value.Y & 0x7FFu) << 11u) | ((Value.Z & 0x3FFu) << 22u);
}

uint32 UE::SVT::PackPageTableEntry(const FIntVector3& Coord)
{
	uint32 Result = (Coord.X & 0xFFu) | ((Coord.Y & 0xFFu) << 8u) | ((Coord.Z & 0xFFu) << 16u);
	return Result;
}

FIntVector3 UE::SVT::UnpackPageTableEntry(uint32 Packed)
{
	FIntVector3 Result;
	Result.X = Packed & 0xFFu;
	Result.Y = (Packed >> 8u) & 0xFFu;
	Result.Z = (Packed >> 16u) & 0xFFu;
	return Result;
}

FVector4f UE::SVT::ReadVoxel(int64 VoxelIndex, const uint8* TileData, EPixelFormat Format)
{
	using namespace UE::SVT::Private;
	if (Format == PF_Unknown)
	{
		return FVector4f(0.0f, 0.0f, 0.0f, 0.0f);
	}
	switch (Format)
	{
	case PF_R8:
	case PF_G8:
		return FVector4f(TileData[VoxelIndex] / 255.0f, 0.0f, 0.0f, 0.0f);
	case PF_R8G8:
		return FVector4f(TileData[VoxelIndex * 2 + 0] / 255.0f, TileData[VoxelIndex * 2 + 1] / 255.0f, 0.0f, 0.0f);
	case PF_R8G8B8A8:
		return FVector4f(TileData[VoxelIndex * 4 + 0] / 255.0f, TileData[VoxelIndex * 4 + 1] / 255.0f, TileData[VoxelIndex * 4 + 2] / 255.0f, TileData[VoxelIndex * 4 + 3] / 255.0f);
	case PF_B8G8R8A8:
		return FVector4f(TileData[VoxelIndex * 4 + 2] / 255.0f, TileData[VoxelIndex * 4 + 1] / 255.0f, TileData[VoxelIndex * 4 + 0] / 255.0f, TileData[VoxelIndex * 4 + 3] / 255.0f);
	case PF_R16F:
		return FVector4f(F16ToF32(((const uint16*)TileData)[VoxelIndex]), 0.0f, 0.0f, 0.0f);
	case PF_G16R16F:
		return FVector4f(F16ToF32(((const uint16*)TileData)[VoxelIndex * 2 + 0]), F16ToF32(((const uint16*)TileData)[VoxelIndex * 2 + 1]), 0.0f, 0.0f);
	case PF_FloatRGBA:
		return FVector4f(F16ToF32(((const uint16*)TileData)[VoxelIndex * 4 + 0]), F16ToF32(((const uint16*)TileData)[VoxelIndex * 4 + 1]), F16ToF32(((const uint16*)TileData)[VoxelIndex * 4 + 2]), F16ToF32(((const uint16*)TileData)[VoxelIndex * 4 + 3]));
	case PF_R32_FLOAT:
		return FVector4f(((const float*)TileData)[VoxelIndex], 0.0f, 0.0f, 0.0f);
	case PF_G32R32F:
		return FVector4f(((const float*)TileData)[VoxelIndex * 2 + 0], ((const float*)TileData)[VoxelIndex * 2 + 1], 0.0f, 0.0f);
	case PF_A32B32G32R32F:
		return FVector4f(((const float*)TileData)[VoxelIndex * 4 + 0], ((const float*)TileData)[VoxelIndex * 4 + 1], ((const float*)TileData)[VoxelIndex * 4 + 2], ((const float*)TileData)[VoxelIndex * 4 + 3]);
	case PF_A2B10G10R10:
	{
		const uint32 Packed = ((const uint32*)TileData)[VoxelIndex];
		return FVector4f(
			((Packed >>  0u) & 0x3FFu) / 1023.0f, 
			((Packed >> 10u) & 0x3FFu) / 1023.0f,
			((Packed >> 20u) & 0x3FFu) / 1023.0f, 
			((Packed >> 30u) & 0x3u) / 3.0f);
	}
	case PF_FloatR11G11B10:
	{
		const uint32 Packed = ((const uint32*)TileData)[VoxelIndex];
		return FVector4f(
			F16ToF32(uint16((Packed >> 17u) & 0x7FF0u)),
			F16ToF32(uint16((Packed >> 6u) & 0x7FF0u)),
			F16ToF32(uint16((Packed << 5u) & 0x7FE0u)),
			0.0f);
	}
	default:
		checkf(false, TEXT("'%s' is not a supported SparseVolumeTexture format!"), GPixelFormats[Format].Name);
		return FVector4f(0.0f, 0.0f, 0.0f, 0.0f);
	}
}

void UE::SVT::WriteVoxel(int64 VoxelIndex, uint8* TileData, EPixelFormat Format, const FVector4f& Value, int32 DstComponent)
{
	using namespace UE::SVT::Private;
	if (Format == PF_Unknown)
	{
		return;
	}
	switch (Format)
	{
	case PF_R8:
	case PF_G8:
		if (DstComponent == -1 || DstComponent == 0) TileData[VoxelIndex] = uint8(FMath::Clamp(Value.X, 0.0f, 1.0f) * 255.0f);
		break;
	case PF_R8G8:
		if (DstComponent == -1 || DstComponent == 0) TileData[VoxelIndex * 2 + 0] = uint8(FMath::Clamp(Value.X, 0.0f, 1.0f) * 255.0f);
		if (DstComponent == -1 || DstComponent == 1) TileData[VoxelIndex * 2 + 1] = uint8(FMath::Clamp(Value.Y, 0.0f, 1.0f) * 255.0f);
		break;
	case PF_R8G8B8A8:
		if (DstComponent == -1 || DstComponent == 0) TileData[VoxelIndex * 4 + 0] = uint8(FMath::Clamp(Value.X, 0.0f, 1.0f) * 255.0f);
		if (DstComponent == -1 || DstComponent == 1) TileData[VoxelIndex * 4 + 1] = uint8(FMath::Clamp(Value.Y, 0.0f, 1.0f) * 255.0f);
		if (DstComponent == -1 || DstComponent == 2) TileData[VoxelIndex * 4 + 2] = uint8(FMath::Clamp(Value.Z, 0.0f, 1.0f) * 255.0f);
		if (DstComponent == -1 || DstComponent == 3) TileData[VoxelIndex * 4 + 3] = uint8(FMath::Clamp(Value.W, 0.0f, 1.0f) * 255.0f);
		break;
	case PF_B8G8R8A8:
		if (DstComponent == -1 || DstComponent == 0) TileData[VoxelIndex * 4 + 2] = uint8(FMath::Clamp(Value.X, 0.0f, 1.0f) * 255.0f);
		if (DstComponent == -1 || DstComponent == 1) TileData[VoxelIndex * 4 + 1] = uint8(FMath::Clamp(Value.Y, 0.0f, 1.0f) * 255.0f);
		if (DstComponent == -1 || DstComponent == 2) TileData[VoxelIndex * 4 + 0] = uint8(FMath::Clamp(Value.Z, 0.0f, 1.0f) * 255.0f);
		if (DstComponent == -1 || DstComponent == 3) TileData[VoxelIndex * 4 + 3] = uint8(FMath::Clamp(Value.W, 0.0f, 1.0f) * 255.0f);
		break;
	case PF_R16F:
		if (DstComponent == -1 || DstComponent == 0) ((uint16*)TileData)[VoxelIndex] = F32ToF16(Value.X);
		break;
	case PF_G16R16F:
		if (DstComponent == -1 || DstComponent == 0) ((uint16*)TileData)[VoxelIndex * 2 + 0] = F32ToF16(Value.X);
		if (DstComponent == -1 || DstComponent == 1) ((uint16*)TileData)[VoxelIndex * 2 + 1] = F32ToF16(Value.Y);
		break;
	case PF_FloatRGBA:
		if (DstComponent == -1 || DstComponent == 0) ((uint16*)TileData)[VoxelIndex * 4 + 0] = F32ToF16(Value.X);
		if (DstComponent == -1 || DstComponent == 1) ((uint16*)TileData)[VoxelIndex * 4 + 1] = F32ToF16(Value.Y);
		if (DstComponent == -1 || DstComponent == 2) ((uint16*)TileData)[VoxelIndex * 4 + 2] = F32ToF16(Value.Z);
		if (DstComponent == -1 || DstComponent == 3) ((uint16*)TileData)[VoxelIndex * 4 + 3] = F32ToF16(Value.W);
		break;
	case PF_R32_FLOAT:
		if (DstComponent == -1 || DstComponent == 0) ((float*)TileData)[VoxelIndex] = Value.X;
		break;
	case PF_G32R32F:
		if (DstComponent == -1 || DstComponent == 0) ((float*)TileData)[VoxelIndex * 2 + 0] = Value.X;
		if (DstComponent == -1 || DstComponent == 1) ((float*)TileData)[VoxelIndex * 2 + 1] = Value.Y;
		break;
	case PF_A32B32G32R32F:
		if (DstComponent == -1 || DstComponent == 0) ((float*)TileData)[VoxelIndex * 4 + 0] = Value.X;
		if (DstComponent == -1 || DstComponent == 1) ((float*)TileData)[VoxelIndex * 4 + 1] = Value.Y;
		if (DstComponent == -1 || DstComponent == 2) ((float*)TileData)[VoxelIndex * 4 + 2] = Value.Z;
		if (DstComponent == -1 || DstComponent == 3) ((float*)TileData)[VoxelIndex * 4 + 3] = Value.W;
		break;
	case PF_A2B10G10R10:
	{
		uint32& Packed = ((uint32*)TileData)[VoxelIndex];
		if (DstComponent == -1 || DstComponent == 0) Packed |= (uint32(FMath::Clamp(Value.X, 0.0f, 1.0f) * 1023.0f) & 0x3FFu) << 0u;
		if (DstComponent == -1 || DstComponent == 1) Packed |= (uint32(FMath::Clamp(Value.Y, 0.0f, 1.0f) * 1023.0f) & 0x3FFu) << 10u;
		if (DstComponent == -1 || DstComponent == 2) Packed |= (uint32(FMath::Clamp(Value.Z, 0.0f, 1.0f) * 1023.0f) & 0x3FFu) << 20u;
		if (DstComponent == -1 || DstComponent == 3) Packed |= (uint32(FMath::Clamp(Value.W, 0.0f, 1.0f) * 3.0f) & 0x3u) << 30u;
		break;
	}
	case PF_FloatR11G11B10:
	{
		uint32& Packed = ((uint32*)TileData)[VoxelIndex];
		if (DstComponent == -1 || DstComponent == 0) Packed |= ((uint32)F32ToF16(Value.X) << 17u) & 0xFFE00000u;
		if (DstComponent == -1 || DstComponent == 1) Packed |= ((uint32)F32ToF16(Value.Y) << 6u) & 0x001FFC00u;
		if (DstComponent == -1 || DstComponent == 2) Packed |= ((uint32)F32ToF16(Value.Z) >> 5u) & 0x000003FFu;
		break;
	}
	default:
		checkf(false, TEXT("'%s' is not a supported SparseVolumeTexture format!"), GPixelFormats[Format].Name);
	}
}

bool UE::SVT::IsSupportedFormat(EPixelFormat Format)
{
	switch (Format)
	{
	case PF_Unknown:
	case PF_R8:
	case PF_G8:
	case PF_R8G8:
	case PF_R8G8B8A8:
	case PF_B8G8R8A8:
	case PF_R16F:
	case PF_G16R16F:
	case PF_FloatRGBA:
	case PF_R32_FLOAT:
	case PF_G32R32F:
	case PF_A32B32G32R32F:
	case PF_A2B10G10R10:
	case PF_FloatR11G11B10:
		return true;
	default:
		return false;
	}
}

bool UE::SVT::IsInBounds(const FIntVector3& Point, const FIntVector3& Min, const FIntVector3& Max)
{
	return Point.X >= Min.X && Point.Y >= Min.Y && Point.Z >= Min.Z
		&& Point.X < Max.X && Point.Y < Max.Y && Point.Z < Max.Z;
}

FIntVector3 UE::SVT::ShiftRightAndMax(const FIntVector3& Value, uint32 ShiftBy, int32 MinValue)
{
	FIntVector3 Result = FIntVector3(
		FMath::Max(Value.X >> ShiftBy, MinValue),
		FMath::Max(Value.Y >> ShiftBy, MinValue),
		FMath::Max(Value.Z >> ShiftBy, MinValue));
	return Result;
}
