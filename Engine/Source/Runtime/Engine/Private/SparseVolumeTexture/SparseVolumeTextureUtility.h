// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "SparseVolumeTexture/SparseVolumeTexture.h"

namespace UE
{
namespace SVT
{
	// We are using this instead of GMaxVolumeTextureDimensions to be independent of the platform that
	// the asset is imported on. 2048 should be a safe value that should be supported by all our platforms.
	static constexpr int32 MaxVolumeTextureDim = 2048;
	static constexpr int64 MaxResourceSize = 2048LL * 1024LL * 1024LL;
	static constexpr int32 NumVoxelsPerTile = SPARSE_VOLUME_TILE_RES * SPARSE_VOLUME_TILE_RES * SPARSE_VOLUME_TILE_RES;
	static constexpr int32 NumVoxelsPerPaddedTile = SPARSE_VOLUME_TILE_RES_PADDED * SPARSE_VOLUME_TILE_RES_PADDED * SPARSE_VOLUME_TILE_RES_PADDED;
	static constexpr int32 NumOccupancyWordsPerPaddedTile = (NumVoxelsPerPaddedTile + 31) / 32;
	static constexpr int32 OccupancyBitsSizePerPaddedTile = NumOccupancyWordsPerPaddedTile * sizeof(uint32);

	uint32 PackX11Y11Z10(const FIntVector3& Value);
	uint32 PackPageTableEntry(const FIntVector3& Coord);
	FIntVector3 UnpackPageTableEntry(uint32 Packed);
	FVector4f ReadVoxel(int64 VoxelIndex, const uint8* TileData, EPixelFormat Format);
	void WriteVoxel(int64 VoxelIndex, uint8* TileData, EPixelFormat Format, const FVector4f& Value, int32 DstComponent = -1);
	bool IsSupportedFormat(EPixelFormat Format);
	// Returns true if Point >= Min && Point < Max for all components
	bool IsInBounds(const FIntVector3& Point, const FIntVector3& Min, const FIntVector3& Max);
	// Computes Max(Value >> ShiftBy, MinValue)
	FIntVector3 ShiftRightAndMax(const FIntVector3& Value, uint32 ShiftBy, int32 MinValue);

namespace Private
{
	// SVT_TODO: This really should be a shared function.
	template <typename Y, typename T>
	void SerializeEnumAs(FArchive& Ar, T& Target)
	{
		Y Buffer = static_cast<Y>(Target);
		Ar << Buffer;
		if (Ar.IsLoading())
		{
			Target = static_cast<T>(Buffer);
		}
	}
} // Private

} // SVT
} // UE
