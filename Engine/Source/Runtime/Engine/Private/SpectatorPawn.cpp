// Copyright Epic Games, Inc. All Rights Reserved.


#include "GameFramework/SpectatorPawn.h"
#include "Async/TaskGraphInterfaces.h"
#include "Components/SphereComponent.h"
#include "GameFramework/SpectatorPawnMovement.h"
#include "GameFramework/WorldSettings.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SpectatorPawn)

ASpectatorPawn::ASpectatorPawn(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer
	.SetDefaultSubobjectClass<USpectatorPawnMovement>(Super::MovementComponentName)
	.DoNotCreateDefaultSubobject(Super::MeshComponentName)
	)
{
	SetCanBeDamaged(false);
	//SetRemoteRoleForBackwardsCompat(ROLE_SimulatedProxy);
	//bReplicates = true;

	BaseEyeHeight = 0.0f;
	bCollideWhenPlacing = false;
	SpawnCollisionHandlingMethod = ESpawnActorCollisionHandlingMethod::AlwaysSpawn;
	MovementComponent->bComponentShouldUpdatePhysicsVolume = false;

	static FName CollisionProfileName("Spectator");
	GetCollisionComponent()->SetCollisionProfileName(CollisionProfileName);
}


void ASpectatorPawn::PossessedBy(class AController* NewController)
{
	if (bReplicates)
	{
		Super::PossessedBy(NewController);
	}
	else
	{
		// We don't want the automatic changing of net role in Pawn code since we don't replicate, so don't call Super.
		AController* const OldController = GetController();
		SetController(NewController);

		// dispatch Blueprint event if necessary
		if (OldController != NewController)
		{
			ReceivePossessed(GetController());
		}
	}
}

void ASpectatorPawn::TurnAtRate(float Rate)
{
	// Replays that use small or zero time dilation to pause will block gamepads from steering the spectator pawn
	AWorldSettings* const WorldSettings = GetWorldSettings();
	if (WorldSettings)
	{
 		float TimeDilation = WorldSettings->GetEffectiveTimeDilation();
		if (TimeDilation <= UE_KINDA_SMALL_NUMBER)
		{
			const float DeltaTime = FApp::GetDeltaTime();
			AddControllerYawInput(Rate * BaseTurnRate * DeltaTime * CustomTimeDilation);
			return;
		}
	}

	Super::TurnAtRate(Rate);
}

void ASpectatorPawn::LookUpAtRate(float Rate)
{
	// Replays that use small or zero time dilation to pause will block gamepads from steering the spectator pawn
	AWorldSettings* const WorldSettings = GetWorldSettings();
	if (WorldSettings)
	{
		float TimeDilation = WorldSettings->GetEffectiveTimeDilation();
		if (TimeDilation <= UE_KINDA_SMALL_NUMBER)
		{
			const float DeltaTime = FApp::GetDeltaTime();
			AddControllerPitchInput(Rate * BaseLookUpRate * DeltaTime * CustomTimeDilation);
			return;
		}
	}

	Super::LookUpAtRate(Rate);
}

