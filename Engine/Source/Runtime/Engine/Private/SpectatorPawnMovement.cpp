// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	SpectatorPawnMovement.cpp: SpectatorPawn movement implementation

=============================================================================*/

#include "GameFramework/SpectatorPawnMovement.h"
#include "Misc/App.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SpectatorPawnMovement)

USpectatorPawnMovement::USpectatorPawnMovement(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	MaxSpeed = 1200.f;
	Acceleration = 4000.f;
	Deceleration = 12000.f;

	ResetMoveState();

	bIgnoreTimeDilation = false;
}


void USpectatorPawnMovement::TickComponent(float DeltaTime, enum ELevelTick TickType, FActorComponentTickFunction* ThisTickFunction)
{
	if (!PawnOwner || !UpdatedComponent)
	{
		return;
	}

	// We might want to ignore time dilation
	const float AdjustedDeltaTime = bIgnoreTimeDilation ? FApp::GetDeltaTime() : DeltaTime;

	Super::TickComponent(AdjustedDeltaTime, TickType, ThisTickFunction);
};


