///////////////////////////////////////////////////////////////////////  
//	SpeedTreeWind.cpp
//
//	*** INTERACTIVE DATA VISUALIZATION (IDV) CONFIDENTIAL AND PROPRIETARY INFORMATION ***
//
//	This software is supplied under the terms of a license agreement or
//	nondisclosure agreement with Interactive Data Visualization, Inc. and
//  may not be copied, disclosed, or exploited except in accordance with 
//  the terms of that agreement.
//
//      Copyright (c) 2003-2012 IDV, Inc.
//      All rights reserved in all media.
//
//      IDV, Inc.
//      http://www.idvinc.com

//  SpeedTree v6.2.2 wind class rewritten for use inside UE with no other dependencies


///////////////////////////////////////////////////////////////////////  
// Preprocessor / Includes

#include "SpeedTreeWind.h"
#include "RHIResources.h"

IMPLEMENT_GLOBAL_SHADER_PARAMETER_STRUCT(FSpeedTreeUniformParameters, "SpeedTreeData");


///////////////////////////////////////////////////////////////////////  
//	FSpeedTreeWind::SBranchWindLevel::SBranchWindLevel

FSpeedTreeWind::SBranchWindLevel::SBranchWindLevel( ) :
	m_fTurbulence(0.3f),
	m_fTwitch(0.75f),
	m_fTwitchFreqScale(0.3f)
{
	for (int32 i = 0; i < FSpeedTreeWind::NUM_WIND_POINTS_IN_CURVE; ++i)
	{
		m_afDistance[i] = 0.0f;
		m_afDirectionAdherence[i] = 0.0f;
		m_afWhip[i] = 0.0f;
	}
}


///////////////////////////////////////////////////////////////////////  
//	FSpeedTreeWind::SWindGroup::SWindGroup

FSpeedTreeWind::SWindGroup::SWindGroup( ) :
	m_fTwitchSharpness(20.0f),
	m_fRollMaxScale(1.0f),
	m_fRollMinScale(1.0f),
	m_fRollSpeed(0.3f),
	m_fRollSeparation(0.005f),
	m_fLeewardScalar(1.0f)
{
	for (int32 i = 0; i < FSpeedTreeWind::NUM_WIND_POINTS_IN_CURVE; ++i)
	{
		m_afRippleDistance[i] = 0.0f;
		m_afTumbleFlip[i] = 0.0f;
		m_afTumbleTwist[i] = 0.0f;
		m_afTumbleDirectionAdherence[i] = 0.0f;
		m_afTwitchThrow[i] = 0.0f;
	}
}


///////////////////////////////////////////////////////////////////////  
//	FSpeedTreeWind::SParams::SParams

FSpeedTreeWind::SParams::SParams( ) :
	m_fStrengthResponse(5.0f),
	m_fDirectionResponse(2.5f),
	m_fAnchorOffset(0.0f),
	m_fAnchorDistanceScale(1.0f),
	m_fGlobalHeight(50.0f),
	m_fGlobalHeightExponent(2.0f),
	m_fFrondRippleTile(10.0f),
	m_fFrondRippleLightingScalar(1.0f),
	m_fRollingNoiseSize(0.005f),
	m_fRollingNoiseTwist(9.0f),
	m_fRollingNoiseTurbulence(32.0f),
	m_fRollingNoisePeriod(0.4f),
	m_fRollingNoiseSpeed(0.05f),
	m_fRollingBranchFieldMin(0.5f),
	m_fRollingBranchLightingAdjust(0.5f),
	m_fRollingBranchVerticalOffset(-0.5f),
	m_fRollingLeafRippleMin(0.5f),
	m_fRollingLeafTumbleMin(0.5f),
	m_fGustFrequency(0.0f),
	m_fGustStrengthMin(0.5f),
	m_fGustStrengthMax(1.0f),
	m_fGustDurationMin(1.0f),
	m_fGustDurationMax(4.0f),
	m_fGustRiseScalar(1.0f),
	m_fGustFallScalar(1.0f)
{
	for (int32 i = 0; i < FSpeedTreeWind::NUM_OSC_COMPONENTS; ++i)
		for (int32 j = 0; j < FSpeedTreeWind::NUM_WIND_POINTS_IN_CURVE; ++j)
			m_afFrequencies[i][j] = 0.0f;

	for (int32 i = 0; i < FSpeedTreeWind::NUM_WIND_POINTS_IN_CURVE; ++i)
	{
		m_afFrondRippleDistance[i] = 0.0f;
		m_afGlobalDistance[i] = 0.0f;
		m_afGlobalDirectionAdherence[i] = 0.0f;
	}
}


///////////////////////////////////////////////////////////////////////  
//	FSpeedTreeWind::FSpeedTreeWind
/// Default constructor for FSpeedTreeWind.

FSpeedTreeWind::FSpeedTreeWind() :
	m_fStrength(0.0f),
	m_fLastTime(-1.0),
	m_fElapsedTime(0.0),
	m_bGustingEnabled(true),
	m_fGust(0.0f),
	m_fGustTarget(0.0f),
	m_fGustRiseTarget(0.0f),
	m_fGustFallTarget(0.0f),
	m_fGustStart(0.0f),
	m_fGustAtStart(1.0f),
	m_fGustFallStart(0.0f),
	m_fStrengthTarget(0.0f),
	m_fStrengthChangeStartTime(0.0f),
	m_fStrengthChangeEndTime(0.0f),
	m_fStrengthAtStart(0.0f),
	m_fDirectionChangeStartTime(0.0f),
	m_fDirectionChangeEndTime(0.0f),
	m_fCombinedStrength(0.0f),
	m_fMaxBranchLevel1Length(0.0f),
	m_bNeedsReload(false)

{
	for (int32 i = 0; i < FSpeedTreeWind::NUM_OSC_COMPONENTS; ++i)
		m_afOscillationTimes[i] = 0.0f;

	for (int32 i = 0; i < FSpeedTreeWind::NUM_WIND_OPTIONS; ++i)
		m_abOptions[i] = false;

	m_afDirectionTarget[0] = m_afDirectionAtStart[0] = m_afDirectionMidTarget[0] = m_afDirection[0] = 1.0f;
	m_afDirectionTarget[1] = m_afDirectionAtStart[1] = m_afDirectionMidTarget[1] = m_afDirection[1] = 0.0f;
	m_afDirectionTarget[2] = m_afDirectionAtStart[2] = m_afDirectionMidTarget[2] = m_afDirection[2] = 0.0f;

	m_afBranchWindAnchor[0] = m_afBranchWindAnchor[1] = m_afBranchWindAnchor[2] = 0.0f;

	m_afRollingOffset[0] = m_afRollingOffset[1] = 0.0f;
}


///////////////////////////////////////////////////////////////////////  
//	FSpeedTreeWind::Advance

void FSpeedTreeWind::Advance(bool bEnabled, double fTime)
{
	// keep track of time
	if (m_fLastTime == -1.0)
		m_fElapsedTime = 0.0;
	else
		m_fElapsedTime = fTime - m_fLastTime;
	m_fLastTime = fTime;

	{
		// Copy values to previous frame's area of the buffer
		FMemory::Memcpy(m_afShaderTable + NUM_SHADER_VALUES, m_afShaderTable, NUM_SHADER_VALUES * sizeof(m_afShaderTable[0]));
	}

	if (bEnabled)
	{
		if (m_bGustingEnabled)
			Gust(fTime);

		// adjust direction
		float fDirectionFactor = 1.0f;
		if (m_fDirectionChangeEndTime != m_fDirectionChangeStartTime)
			fDirectionFactor = FMath::Min(1.0, FMath::Max(0.0, (fTime - m_fDirectionChangeStartTime) / (m_fDirectionChangeEndTime - m_fDirectionChangeStartTime)));
		fDirectionFactor = LinearSigmoid(fDirectionFactor, 0.5f);

		if (fDirectionFactor < 0.5f)
		{
			// go toward the mid-vector (the mid vector prevents fast swoops when making 180 degree direction changes)
			fDirectionFactor *= 2.0f;
			m_afDirection[0] = Interpolate(m_afDirectionAtStart[0], m_afDirectionMidTarget[0], fDirectionFactor);
			m_afDirection[1] = Interpolate(m_afDirectionAtStart[1], m_afDirectionMidTarget[1], fDirectionFactor);
			m_afDirection[2] = Interpolate(m_afDirectionAtStart[2], m_afDirectionMidTarget[2], fDirectionFactor);
		}
		else
		{
			// go away from the mid-vector (the mid vector prevents fast swoops when making 180 degree direction changes)
			fDirectionFactor = (fDirectionFactor - 0.5f) * 2.0f;
			m_afDirection[0] = Interpolate(m_afDirectionMidTarget[0], m_afDirectionTarget[0], fDirectionFactor);
			m_afDirection[1] = Interpolate(m_afDirectionMidTarget[1], m_afDirectionTarget[1], fDirectionFactor);
			m_afDirection[2] = Interpolate(m_afDirectionMidTarget[2], m_afDirectionTarget[2], fDirectionFactor);
		}
		Normalize(m_afDirection);

		// adjust strength
		float fStrengthFactor = 1.0f;
		if (m_fStrengthChangeEndTime != m_fStrengthChangeStartTime)
			fStrengthFactor = FMath::Min(1.0, FMath::Max(0.0, (fTime - m_fStrengthChangeStartTime) / (m_fStrengthChangeEndTime - m_fStrengthChangeStartTime)));
		else
			fStrengthFactor = 0.0f;

		m_fStrength = Interpolate(m_fStrengthAtStart, m_fStrengthTarget, LinearSigmoid(fStrengthFactor, 0.0f));

		// combine it with the gust value
		m_fCombinedStrength = FMath::Max(0.0f, FMath::Min(1.0f, m_fStrength + m_fGust));

		// update the rolling wind offset
		m_afRollingOffset[0] += m_afDirection[0] * m_fCombinedStrength * m_sParams.m_fRollingNoiseSpeed * m_fElapsedTime;
		m_afRollingOffset[1] += m_afDirection[1] * m_fCombinedStrength * m_sParams.m_fRollingNoiseSpeed * m_fElapsedTime;

		// compute oscillation indices
		float fIndex = m_fCombinedStrength * ((float)FSpeedTreeWind::NUM_WIND_POINTS_IN_CURVE - 1.0f);
		int32 nBefore = static_cast<int>(fIndex);
		int32 nAfter = nBefore + 1;
		float fInterpolate;
		if (nAfter >= FSpeedTreeWind::NUM_WIND_POINTS_IN_CURVE)
		{
			nBefore = nAfter = FSpeedTreeWind::NUM_WIND_POINTS_IN_CURVE - 1;
			fInterpolate = 0.0f;
		}
		else if (nBefore < 0)
		{
			nBefore = nAfter = 0;
			fInterpolate = 0.0f;
		}
		else
		{
			fInterpolate = fIndex - static_cast<float>(nBefore);
		}

		// update oscillation times
		for (int32 i = 0; i < NUM_OSC_COMPONENTS; ++i)
		{
			m_afOscillationTimes[i] += m_fElapsedTime * Interpolate(m_sParams.m_afFrequencies[i][nBefore], m_sParams.m_afFrequencies[i][nAfter], fInterpolate);
		}

		// set shader values
		m_afShaderTable[SH_WIND_DIR_X] = m_afDirection[0];
		m_afShaderTable[SH_WIND_DIR_Y] = m_afDirection[1]; 
		m_afShaderTable[SH_WIND_DIR_Z] = m_afDirection[2];

		// general
		m_afShaderTable[SH_GENERAL_STRENGTH] = m_fCombinedStrength;

		// global
		m_afShaderTable[SH_GLOBAL_DISTANCE] = Interpolate(m_sParams.m_afGlobalDistance[nBefore], m_sParams.m_afGlobalDistance[nAfter], fInterpolate);
		m_afShaderTable[SH_GLOBAL_TIME] = m_afOscillationTimes[OSC_GLOBAL];
		m_afShaderTable[SH_GLOBAL_HEIGHT] = m_sParams.m_fGlobalHeight != 0.0f ? (1.0f / m_sParams.m_fGlobalHeight) : 1.0f;
		m_afShaderTable[SH_GLOBAL_HEIGHT_EXPONENT] = m_sParams.m_fGlobalHeightExponent;
		m_afShaderTable[SH_GLOBAL_DIRECTION_ADHERENCE] = Interpolate(m_sParams.m_afGlobalDirectionAdherence[nBefore], m_sParams.m_afGlobalDirectionAdherence[nAfter], fInterpolate);

		// branch
		m_afShaderTable[SH_BRANCH_1_DISTANCE] = Interpolate(m_sParams.m_asBranch[0].m_afDistance[nBefore], m_sParams.m_asBranch[0].m_afDistance[nAfter], fInterpolate);
		m_afShaderTable[SH_BRANCH_1_TIME] = m_afOscillationTimes[OSC_BRANCH_1];
		m_afShaderTable[SH_BRANCH_2_DISTANCE] = Interpolate(m_sParams.m_asBranch[1].m_afDistance[nBefore], m_sParams.m_asBranch[1].m_afDistance[nAfter], fInterpolate);
		m_afShaderTable[SH_BRANCH_2_TIME] = m_afOscillationTimes[OSC_BRANCH_2];

		float afComputedWindAnchor[3];
		ComputeWindAnchor(afComputedWindAnchor);
		m_afShaderTable[SH_WIND_ANCHOR_X] = afComputedWindAnchor[0];
		m_afShaderTable[SH_WIND_ANCHOR_Y] = afComputedWindAnchor[1];
		m_afShaderTable[SH_WIND_ANCHOR_Z] = afComputedWindAnchor[2];

		m_afShaderTable[SH_BRANCH_1_DIRECTION_ADHERENCE] = Interpolate(m_sParams.m_asBranch[0].m_afDirectionAdherence[nBefore], m_sParams.m_asBranch[0].m_afDirectionAdherence[nAfter], fInterpolate);
		if (m_abOptions[BRANCH_DIRECTIONAL_1])
			m_afShaderTable[SH_BRANCH_1_DIRECTION_ADHERENCE] *= m_fMaxBranchLevel1Length;
		m_afShaderTable[SH_BRANCH_1_TURBULENCE] = m_sParams.m_asBranch[0].m_fTurbulence;
		m_afShaderTable[SH_BRANCH_2_DIRECTION_ADHERENCE] = Interpolate(m_sParams.m_asBranch[1].m_afDirectionAdherence[nBefore], m_sParams.m_asBranch[1].m_afDirectionAdherence[nAfter], fInterpolate);
		if (m_abOptions[BRANCH_DIRECTIONAL_2])
			m_afShaderTable[SH_BRANCH_2_DIRECTION_ADHERENCE] *= m_fMaxBranchLevel1Length;
		m_afShaderTable[SH_BRANCH_2_TURBULENCE] = m_sParams.m_asBranch[1].m_fTurbulence;

		m_afShaderTable[SH_BRANCH_1_TWITCH] = m_sParams.m_asBranch[0].m_fTwitch;
		m_afShaderTable[SH_BRANCH_1_TWITCH_FREQ_SCALE] = m_sParams.m_asBranch[0].m_fTwitchFreqScale;

		m_afShaderTable[SH_BRANCH_2_TWITCH] = m_sParams.m_asBranch[1].m_fTwitch;
		m_afShaderTable[SH_BRANCH_2_TWITCH_FREQ_SCALE] = m_sParams.m_asBranch[1].m_fTwitchFreqScale;

		m_afShaderTable[SH_BRANCH_1_WHIP] = Interpolate(m_sParams.m_asBranch[0].m_afWhip[nBefore], m_sParams.m_asBranch[0].m_afWhip[nAfter], fInterpolate);
		m_afShaderTable[SH_BRANCH_2_WHIP] = Interpolate(m_sParams.m_asBranch[1].m_afWhip[nBefore], m_sParams.m_asBranch[1].m_afWhip[nAfter], fInterpolate);

		// leaf ripple
		m_afShaderTable[SH_LEAF_1_RIPPLE_TIME] = m_afOscillationTimes[OSC_LEAF_1_RIPPLE];
		m_afShaderTable[SH_LEAF_1_RIPPLE_DISTANCE] = Interpolate(m_sParams.m_asLeaf[0].m_afRippleDistance[nBefore], m_sParams.m_asLeaf[0].m_afRippleDistance[nAfter], fInterpolate);

		m_afShaderTable[SH_LEAF_2_RIPPLE_TIME] = m_afOscillationTimes[OSC_LEAF_2_RIPPLE];
		m_afShaderTable[SH_LEAF_2_RIPPLE_DISTANCE] = Interpolate(m_sParams.m_asLeaf[1].m_afRippleDistance[nBefore], m_sParams.m_asLeaf[1].m_afRippleDistance[nAfter], fInterpolate);

		// leaf tumble
		m_afShaderTable[SH_LEAF_1_TUMBLE_TIME] = m_afOscillationTimes[OSC_LEAF_1_TUMBLE];
		m_afShaderTable[SH_LEAF_1_TUMBLE_FLIP] = Interpolate(m_sParams.m_asLeaf[0].m_afTumbleFlip[nBefore], m_sParams.m_asLeaf[0].m_afTumbleFlip[nAfter], fInterpolate);
		m_afShaderTable[SH_LEAF_1_TUMBLE_TWIST] = Interpolate(m_sParams.m_asLeaf[0].m_afTumbleTwist[nBefore], m_sParams.m_asLeaf[0].m_afTumbleTwist[nAfter], fInterpolate);
		m_afShaderTable[SH_LEAF_1_TUMBLE_DIRECTION_ADHERENCE] = Interpolate(m_sParams.m_asLeaf[0].m_afTumbleDirectionAdherence[nBefore], m_sParams.m_asLeaf[0].m_afTumbleDirectionAdherence[nAfter], fInterpolate);

		m_afShaderTable[SH_LEAF_2_TUMBLE_TIME] = m_afOscillationTimes[OSC_LEAF_2_TUMBLE];
		m_afShaderTable[SH_LEAF_2_TUMBLE_FLIP] = Interpolate(m_sParams.m_asLeaf[1].m_afTumbleFlip[nBefore], m_sParams.m_asLeaf[1].m_afTumbleFlip[nAfter], fInterpolate);
		m_afShaderTable[SH_LEAF_2_TUMBLE_TWIST] = Interpolate(m_sParams.m_asLeaf[1].m_afTumbleTwist[nBefore], m_sParams.m_asLeaf[1].m_afTumbleTwist[nAfter], fInterpolate);
		m_afShaderTable[SH_LEAF_2_TUMBLE_DIRECTION_ADHERENCE] = Interpolate(m_sParams.m_asLeaf[1].m_afTumbleDirectionAdherence[nBefore], m_sParams.m_asLeaf[1].m_afTumbleDirectionAdherence[nAfter], fInterpolate);

		m_afShaderTable[SH_LEAF_1_TWITCH_THROW] = Interpolate(m_sParams.m_asLeaf[0].m_afTwitchThrow[nBefore], m_sParams.m_asLeaf[0].m_afTwitchThrow[nAfter], fInterpolate);
		const float c_fTwitchSharpness1Denom = Interpolate(m_sParams.m_afFrequencies[OSC_LEAF_1_TWITCH][nBefore], m_sParams.m_afFrequencies[OSC_LEAF_1_TWITCH][nAfter], fInterpolate);
		if (c_fTwitchSharpness1Denom < FLT_EPSILON)
			m_afShaderTable[SH_LEAF_1_TWITCH_SHARPNESS] = 0.0f;
		else
			m_afShaderTable[SH_LEAF_1_TWITCH_SHARPNESS] = (1.0f / c_fTwitchSharpness1Denom) * m_sParams.m_asLeaf[0].m_fTwitchSharpness * 10.0f;
		m_afShaderTable[SH_LEAF_1_TWITCH_TIME] = m_afOscillationTimes[OSC_LEAF_1_TWITCH];

		m_afShaderTable[SH_LEAF_2_TWITCH_THROW] = Interpolate(m_sParams.m_asLeaf[1].m_afTwitchThrow[nBefore], m_sParams.m_asLeaf[1].m_afTwitchThrow[nAfter], fInterpolate);
		const float c_fTwitchSharpness2Denom = Interpolate(m_sParams.m_afFrequencies[OSC_LEAF_2_TWITCH][nBefore], m_sParams.m_afFrequencies[OSC_LEAF_2_TWITCH][nAfter], fInterpolate);
		if (c_fTwitchSharpness2Denom < FLT_EPSILON)
			m_afShaderTable[SH_LEAF_2_TWITCH_SHARPNESS] = 0.0f;
		else
			m_afShaderTable[SH_LEAF_2_TWITCH_SHARPNESS] = (1.0f / c_fTwitchSharpness2Denom) * m_sParams.m_asLeaf[1].m_fTwitchSharpness * 10.0f;
		m_afShaderTable[SH_LEAF_2_TWITCH_TIME] = m_afOscillationTimes[OSC_LEAF_2_TWITCH];

		// occlusion
		m_afShaderTable[SH_LEAF_1_LEEWARD_SCALAR] = m_sParams.m_asLeaf[0].m_fLeewardScalar;
		m_afShaderTable[SH_LEAF_2_LEEWARD_SCALAR] = m_sParams.m_asLeaf[1].m_fLeewardScalar;
	
		// frond ripple
		m_afShaderTable[SH_FROND_RIPPLE_TIME] = m_afOscillationTimes[OSC_FROND_RIPPLE];
		m_afShaderTable[SH_FROND_RIPPLE_DISTANCE] = Interpolate(m_sParams.m_afFrondRippleDistance[nBefore], m_sParams.m_afFrondRippleDistance[nAfter], fInterpolate);
		m_afShaderTable[SH_FROND_RIPPLE_TILE] = m_sParams.m_fFrondRippleTile;
		m_afShaderTable[SH_FROND_RIPPLE_LIGHTING_SCALAR] = m_sParams.m_fFrondRippleLightingScalar;

		// rolling
		m_afShaderTable[SH_ROLLING_NOISE_SIZE] = m_sParams.m_fRollingNoiseSize;
		m_afShaderTable[SH_ROLLING_NOISE_TWIST] = m_sParams.m_fRollingNoiseTwist;
		m_afShaderTable[SH_ROLLING_NOISE_TURBULENCE] = m_sParams.m_fRollingNoiseTurbulence;
		m_afShaderTable[SH_ROLLING_NOISE_PERIOD] = m_sParams.m_fRollingNoisePeriod;
		m_afShaderTable[SH_ROLLING_LEAF_RIPPLE_MIN] = m_sParams.m_fRollingLeafRippleMin;
		m_afShaderTable[SH_ROLLING_LEAF_TUMBLE_MIN] = m_sParams.m_fRollingLeafTumbleMin;
		m_afShaderTable[SH_ROLLING_BRANCH_FIELD_MIN] = m_sParams.m_fRollingBranchFieldMin;
		m_afShaderTable[SH_ROLLING_BRANCH_LIGHTING_ADJUST] = m_sParams.m_fRollingBranchLightingAdjust;
		m_afShaderTable[SH_ROLLING_BRANCH_VERTICAL_OFFSET] = m_sParams.m_fRollingBranchVerticalOffset;
		m_afShaderTable[SH_ROLLING_X] = m_afRollingOffset[0];
		m_afShaderTable[SH_ROLLING_Y] = m_afRollingOffset[1];
	}
	else
	{
		m_afShaderTable[SH_WIND_DIR_X] = 1.0f;
		m_afShaderTable[SH_WIND_DIR_Y] = 0.0f;
		m_afShaderTable[SH_WIND_DIR_Z] = 0.0f;

		// general
		m_afShaderTable[SH_GENERAL_STRENGTH] = 0.0f;

		// global
		m_afShaderTable[SH_GLOBAL_DISTANCE] = 0.0f;
		m_afShaderTable[SH_GLOBAL_TIME] = 0.0f;
		m_afShaderTable[SH_GLOBAL_HEIGHT] = 1.0f;
		m_afShaderTable[SH_GLOBAL_HEIGHT_EXPONENT] = 1.0f;
		m_afShaderTable[SH_GLOBAL_DIRECTION_ADHERENCE] = 0.0f;

		// branch
		m_afShaderTable[SH_BRANCH_1_DISTANCE] = 0.0f;
		m_afShaderTable[SH_BRANCH_1_TIME] = 0.0f;
		m_afShaderTable[SH_BRANCH_2_DISTANCE] = 0.0f;
		m_afShaderTable[SH_BRANCH_2_TIME] = 0.0f;

		m_afShaderTable[SH_WIND_ANCHOR_X] = 0.0f;
		m_afShaderTable[SH_WIND_ANCHOR_Y] = 0.0f;
		m_afShaderTable[SH_WIND_ANCHOR_Z] = 0.0f;
		m_afShaderTable[SH_BRANCH_1_TURBULENCE] = 0.0f;
		m_afShaderTable[SH_BRANCH_2_TURBULENCE] = 0.0f;
		m_afShaderTable[SH_BRANCH_1_DIRECTION_ADHERENCE] = 0.0f;
		m_afShaderTable[SH_BRANCH_2_DIRECTION_ADHERENCE] = 0.0f;
		m_afShaderTable[SH_BRANCH_1_TWITCH] = 0.0f;
		m_afShaderTable[SH_BRANCH_1_TWITCH_FREQ_SCALE] = 0.0f;
		m_afShaderTable[SH_BRANCH_2_TWITCH] = 0.0f;
		m_afShaderTable[SH_BRANCH_2_TWITCH_FREQ_SCALE] = 0.0f;
		m_afShaderTable[SH_BRANCH_1_WHIP] = 0.0f;
		m_afShaderTable[SH_BRANCH_2_WHIP] = 0.0f;

		// leaf ripple
		m_afShaderTable[SH_LEAF_1_RIPPLE_TIME] = 0.0f;
		m_afShaderTable[SH_LEAF_1_RIPPLE_DISTANCE] = 0.0f;
		m_afShaderTable[SH_LEAF_2_RIPPLE_TIME] = 0.0f;
		m_afShaderTable[SH_LEAF_2_RIPPLE_DISTANCE] = 0.0f;

		// leaf tumble
		m_afShaderTable[SH_LEAF_1_TUMBLE_TIME] = 0.0f;
		m_afShaderTable[SH_LEAF_1_TUMBLE_FLIP] = 0.0f;
		m_afShaderTable[SH_LEAF_1_TUMBLE_TWIST] = 0.0f;
		m_afShaderTable[SH_LEAF_1_TUMBLE_DIRECTION_ADHERENCE] = 0.0f;
		m_afShaderTable[SH_LEAF_2_TUMBLE_TIME] = 0.0f;
		m_afShaderTable[SH_LEAF_2_TUMBLE_FLIP] = 0.0f;
		m_afShaderTable[SH_LEAF_2_TUMBLE_TWIST] = 0.0f;
		m_afShaderTable[SH_LEAF_2_TUMBLE_DIRECTION_ADHERENCE] = 0.0f;

		// twitch
		m_afShaderTable[SH_LEAF_1_TWITCH_THROW] = 0.0f;
		m_afShaderTable[SH_LEAF_1_TWITCH_SHARPNESS] = 0.0f;
		m_afShaderTable[SH_LEAF_1_TWITCH_TIME] = 0.0f;

		m_afShaderTable[SH_LEAF_2_TWITCH_THROW] = 0.0f;
		m_afShaderTable[SH_LEAF_2_TWITCH_SHARPNESS] = 0.0f;
		m_afShaderTable[SH_LEAF_2_TWITCH_TIME] = 0.0f;

		// occlusion
		m_afShaderTable[SH_LEAF_1_LEEWARD_SCALAR] = 1.0f;
		m_afShaderTable[SH_LEAF_2_LEEWARD_SCALAR] = 1.0f;

		// frond ripple
		m_afShaderTable[SH_FROND_RIPPLE_TIME] = 0.0f;
		m_afShaderTable[SH_FROND_RIPPLE_DISTANCE] = 0.0f;
		m_afShaderTable[SH_FROND_RIPPLE_TILE] = 0.0f;
		m_afShaderTable[SH_FROND_RIPPLE_LIGHTING_SCALAR] = 1.0f;

		// rolling
		m_afShaderTable[SH_ROLLING_NOISE_SIZE] = m_sParams.m_fRollingNoiseSize;
		m_afShaderTable[SH_ROLLING_NOISE_TWIST] = m_sParams.m_fRollingNoiseTwist;
		m_afShaderTable[SH_ROLLING_NOISE_TURBULENCE] = m_sParams.m_fRollingNoiseTurbulence;
		m_afShaderTable[SH_ROLLING_NOISE_PERIOD] = m_sParams.m_fRollingNoisePeriod;
		m_afShaderTable[SH_ROLLING_LEAF_RIPPLE_MIN] = m_sParams.m_fRollingLeafRippleMin;
		m_afShaderTable[SH_ROLLING_LEAF_TUMBLE_MIN] = m_sParams.m_fRollingLeafTumbleMin;
		m_afShaderTable[SH_ROLLING_BRANCH_FIELD_MIN] = m_sParams.m_fRollingBranchFieldMin;
		m_afShaderTable[SH_ROLLING_BRANCH_LIGHTING_ADJUST] = m_sParams.m_fRollingBranchLightingAdjust;
		m_afShaderTable[SH_ROLLING_BRANCH_VERTICAL_OFFSET] = m_sParams.m_fRollingBranchVerticalOffset;
		m_afShaderTable[SH_ROLLING_X] = 0.0f;
		m_afShaderTable[SH_ROLLING_Y] = 0.0f;
	}
}


/////////////////////////////////////////////////////////////////////////////
// FSpeedTreeWind::SetStrength
/// Sets the desired strength (\a fStrength).  The FSpeedTreeWind::Advance( ) function will make the actual strength get there smoothly based on the strength response time.

void FSpeedTreeWind::SetStrength(float fStrength)
{
	if (fStrength != m_fStrength)
	{
		m_fStrengthChangeStartTime = m_fLastTime;

		float fAmount = Interpolate(m_sParams.m_fStrengthResponse * 0.5f, m_sParams.m_fStrengthResponse, fabs(fStrength - m_fStrength));
		m_fStrengthChangeEndTime = m_fStrengthChangeStartTime + fAmount;
		m_fStrengthAtStart = m_fStrength;
		m_fStrengthTarget = fStrength;
	}
}


///////////////////////////////////////////////////////////////////////
// Helper function: ScaleWindCurve

static void ScaleWindCurve(float afCurve[FSpeedTreeWind::NUM_WIND_POINTS_IN_CURVE], float fScalar)
{
	for (int32 i = 0; i < FSpeedTreeWind::NUM_WIND_POINTS_IN_CURVE; ++i)
		afCurve[i] *= fScalar;
}


//////////////////////////////////////////////////////////////////////////
// FSpeedTreeWind::Scale

void FSpeedTreeWind::Scale(float fScalar)
{
	ScaleWindCurve(m_sParams.m_afGlobalDistance, fScalar);
	ScaleWindCurve(m_sParams.m_afFrondRippleDistance, fScalar);

	for (int32 i = 0; i < FSpeedTreeWind::NUM_BRANCH_LEVELS; ++i)
		ScaleWindCurve(m_sParams.m_asBranch[i].m_afDistance, fScalar);

	for (int32 i = 0; i < FSpeedTreeWind::NUM_LEAF_GROUPS; ++i)
	{
		ScaleWindCurve(m_sParams.m_asLeaf[i].m_afRippleDistance, fScalar);
		ScaleWindCurve(m_sParams.m_asLeaf[i].m_afTwitchThrow, fScalar);
		if (fScalar != 0.0f)
			m_sParams.m_asLeaf[i].m_fRollSeparation /= fScalar;
	}

	m_sParams.m_fGlobalHeight *= fScalar;

	m_fMaxBranchLevel1Length *= fScalar;
	m_sParams.m_fAnchorDistanceScale *= fScalar;
	m_sParams.m_fAnchorOffset *= fScalar;
	m_afBranchWindAnchor[0] = fScalar;
	m_afBranchWindAnchor[1] = fScalar;
	m_afBranchWindAnchor[2] = fScalar;
}


/////////////////////////////////////////////////////////////////////////////
// FSpeedTreeWind::SetDirection

void FSpeedTreeWind::SetDirection(const FVector& vDir)
{
	if (vDir.X != m_afDirection[0] ||
		vDir.Y != m_afDirection[1] ||
		vDir.Z != m_afDirection[2])
	{
		m_afDirectionTarget[0] = vDir.X;
		m_afDirectionTarget[1] = vDir.Y;
		m_afDirectionTarget[2] = vDir.Z;

		float fDot = m_afDirection[0] * vDir.X + m_afDirection[1] * vDir.Y + m_afDirection[2] * vDir.Z;
		float fDistanceToTravel = 1.0f - ((fDot + 1.0f) * 0.5f);

		m_fDirectionChangeStartTime = m_fLastTime;
		float fAmount = Interpolate(m_sParams.m_fDirectionResponse * 0.5f, m_sParams.m_fDirectionResponse, fDistanceToTravel);
		m_fDirectionChangeEndTime = m_fDirectionChangeStartTime + fAmount;

		m_afDirectionAtStart[0] = m_afDirection[0];
		m_afDirectionAtStart[1] = m_afDirection[1];
		m_afDirectionAtStart[2] = m_afDirection[2];

		m_afDirectionMidTarget[0] = (m_afDirectionAtStart[0] + m_afDirectionTarget[0]) * 0.5f;
		m_afDirectionMidTarget[1] = (m_afDirectionAtStart[1] + m_afDirectionTarget[1]) * 0.5f;
		m_afDirectionMidTarget[2] = (m_afDirectionAtStart[2] + m_afDirectionTarget[2]) * 0.5f;
		Normalize(m_afDirectionMidTarget);
	}
}

void FSpeedTreeWind::SetGustMin(float InGustMin)
{
	m_sParams.m_fGustStrengthMin = InGustMin;
}

void FSpeedTreeWind::SetGustMax(float InGustMax)
{
	m_sParams.m_fGustStrengthMax = InGustMax;
}


/////////////////////////////////////////////////////////////////////////////
// FSpeedTreeWind::SetInitDirection

void FSpeedTreeWind::SetInitDirection(const FVector& vDir)
{
	m_afDirectionTarget[0] = m_afDirectionAtStart[0] = m_afDirectionMidTarget[0] = m_afDirection[0] = vDir.X;
	m_afDirectionTarget[1] = m_afDirectionAtStart[1] = m_afDirectionMidTarget[1] = m_afDirection[1] = vDir.Y;
	m_afDirectionTarget[2] = m_afDirectionAtStart[2] = m_afDirectionMidTarget[2] = m_afDirection[2] = vDir.Z;
}


/////////////////////////////////////////////////////////////////////////////
// FSpeedTreeWind::Gust
/// Advances the gust parameter and randomly gusts if it can (e.g., the gust is not dying off or rising) based on the gust frequency.

void FSpeedTreeWind::Gust(double fTime)
{
	const float c_fGustAdjust = 0.01f;
	if (fTime > m_fGustFallTarget || (fTime < m_fGustFallStart && fTime > m_fGustRiseTarget))
	{
		// it is legal to gust (you can't gust on the way out of a gust to prevent jerks)
		if (RandomFloat(0.0f, m_fElapsedTime) < (m_fElapsedTime * m_sParams.m_fGustFrequency * c_fGustAdjust))
		{
			// we got one, set it up
			m_fGustStart = fTime;
			m_fGustAtStart = m_fGust;
			m_fGustTarget = RandomFloat(m_sParams.m_fGustStrengthMin, m_sParams.m_fGustStrengthMax);
			if (m_fGustTarget > 1.0f - m_fStrength)
				m_fGustTarget = 1.0f - m_fStrength;

			float fAmount = Interpolate(m_sParams.m_fStrengthResponse * 0.5f, m_sParams.m_fStrengthResponse, fabs(m_fGustTarget - m_fStrength));
			if (m_fGustTarget > m_fGust)
				m_fGustRiseTarget = fTime + m_sParams.m_fGustRiseScalar * RandomFloat(fAmount * 1.0f, fAmount * 2.0f);
			else
				m_fGustRiseTarget = fTime + m_sParams.m_fGustFallScalar * RandomFloat(fAmount, fAmount * 2.0f);

			m_fGustFallStart = m_fGustRiseTarget + RandomFloat(m_sParams.m_fGustDurationMin, m_sParams.m_fGustDurationMax);
			m_fGustFallTarget = m_fGustFallStart + m_sParams.m_fGustFallScalar * RandomFloat(fAmount * 2.0f, fAmount * 3.0f);
		}
	}

	if (fTime < m_fGustRiseTarget)
	{
		// s-curve toward the target
		m_fGust = Interpolate(m_fGustAtStart, m_fGustTarget, LinearSigmoid((fTime - m_fGustStart) / (m_fGustRiseTarget - m_fGustStart), 0.0f));
	}
	else if (fTime > m_fGustFallStart && m_fGustFallTarget > 0.0f && m_fGustFallTarget > m_fGustFallStart)
	{
		// s-curve back to zero
		m_fGust = Interpolate(m_fGustTarget, 0.0f, LinearSigmoid((fTime - m_fGustFallStart) / (m_fGustFallTarget - m_fGustFallStart), 0.5f));
	}

	m_fGust = FMath::Max(0.0f, FMath::Min(1.0f, m_fGust));
}


/////////////////////////////////////////////////////////////////////////////
// FSpeedTreeWind::ComputeWindAnchor
/// Computes the wind anchor position based on current wind conditions.

void FSpeedTreeWind::ComputeWindAnchor(float* pPos)
{
	float afDirection[3] = { m_afDirection[0], m_afDirection[1], m_afDirection[2] + m_sParams.m_fAnchorOffset };
	Normalize(afDirection);

	pPos[0] = m_afBranchWindAnchor[0] + afDirection[0] * m_fMaxBranchLevel1Length * m_sParams.m_fAnchorDistanceScale;
	pPos[1] = m_afBranchWindAnchor[1] + afDirection[1] * m_fMaxBranchLevel1Length * m_sParams.m_fAnchorDistanceScale;
	pPos[2] = m_afBranchWindAnchor[2] + afDirection[2] * m_fMaxBranchLevel1Length * m_sParams.m_fAnchorDistanceScale;
}


///////////////////////////////////////////////////////////////////////
// FSpeedTreeWind::RandomFloat

FORCEINLINE float FSpeedTreeWind::RandomFloat(float fMin, float fMax) const
{
	return FMath::FRandRange(fMin, fMax);
}


///////////////////////////////////////////////////////////////////////
// FSpeedTreeWind::LinearSigmoid
/// Converts an input value (\a fInput) in the range [0.0, 1.0] to an s-curve.  The parameter \a fLinearness flattens out the s-curve where 0.0 = s-curve and 1.0 = linear.

FORCEINLINE float FSpeedTreeWind::LinearSigmoid(float fInput, float fLinearness)
{
	float fSigmoid = 1.0f / (1.0f + exp(-Interpolate(-6.0f, 6.0f, fInput)));
	return Interpolate(fSigmoid, fInput, fLinearness);
}


///////////////////////////////////////////////////////////////////////
// FSpeedTreeWind::SetParams

void FSpeedTreeWind::SetParams(const FSpeedTreeWind::SParams& sParams)
{
	m_sParams = sParams;
}


///////////////////////////////////////////////////////////////////////
// FSpeedTreeWind::GetParams

const FSpeedTreeWind::SParams& FSpeedTreeWind::GetParams(void) const
{
	return m_sParams;
}


///////////////////////////////////////////////////////////////////////
// FSpeedTreeWind::EnableGusting

void FSpeedTreeWind::EnableGusting(bool bEnabled)
{
	m_bGustingEnabled = bEnabled;
}


///////////////////////////////////////////////////////////////////////
// FSpeedTreeWind::SetGustFrequency

void FSpeedTreeWind::SetGustFrequency(float fGustFreq)
{
	m_sParams.m_fGustFrequency = fGustFreq;
}


///////////////////////////////////////////////////////////////////////
// FSpeedTreeWind::SetTreeValues

void FSpeedTreeWind::SetTreeValues(const FVector& vBranchAnchor, float fMaxBranchLength)
{
	m_afBranchWindAnchor[0] = vBranchAnchor.X;
	m_afBranchWindAnchor[1] = vBranchAnchor.Y;
	m_afBranchWindAnchor[2] = vBranchAnchor.Z;
	m_fMaxBranchLevel1Length = fMaxBranchLength;
}


///////////////////////////////////////////////////////////////////////
// FSpeedTreeWind::GetBranchAnchor

const float* FSpeedTreeWind::GetBranchAnchor(void) const
{
	return m_afBranchWindAnchor;
}


///////////////////////////////////////////////////////////////////////
// FSpeedTreeWind::GetMaxBranchLength

float FSpeedTreeWind::GetMaxBranchLength(void) const
{
	return m_fMaxBranchLevel1Length;
}


///////////////////////////////////////////////////////////////////////
// FSpeedTreeWind::Normalize
/// Normalizes the the incoming vector (\a pVector).

inline void FSpeedTreeWind::Normalize(float* pVector)
{
	float fMagnitude = FMath::Sqrt(pVector[0] * pVector[0] + pVector[1] * pVector[1] + pVector[2] * pVector[2]);
	if (fMagnitude != 0.0f)
	{
		pVector[0] /= fMagnitude;
		pVector[1] /= fMagnitude;
		pVector[2] /= fMagnitude;
	}
	else
	{
		pVector[0] = 0.0f;
		pVector[1] = 0.0f;
		pVector[2] = 0.0f;
	}
}


///////////////////////////////////////////////////////////////////////
// FSpeedTreeWind::Interpolate

FORCEINLINE float FSpeedTreeWind::Interpolate(float fA, float fB, float fAmt)
{
	return (fA + (fB - fA) * fAmt);
}


///////////////////////////////////////////////////////////////////////
//  FSpeedTreeWind::SetOption

void FSpeedTreeWind::SetOption(EOptions eOption, bool bState)
{ 
	m_abOptions[eOption] = bState; 
}


///////////////////////////////////////////////////////////////////////
//  FSpeedTreeWind::IsOptionEnabled

bool FSpeedTreeWind::IsOptionEnabled(EOptions eOption) const
{ 
	return m_abOptions[eOption];
}


///////////////////////////////////////////////////////////////////////
//  FSpeedTreeWind::GetShaderTable

const float* FSpeedTreeWind::GetShaderTable(void) const
{ 
	return m_afShaderTable;
}


///////////////////////////////////////////////////////////////////////
//  Serializer

FArchive& operator<<(FArchive& Ar, FSpeedTreeWind& Wind)
{
	FSpeedTreeWind::SParams Params = Wind.GetParams();

	#define SERIALIZE_CURVE(name) for (int32 CurveIndex = 0; CurveIndex < FSpeedTreeWind::NUM_WIND_POINTS_IN_CURVE; ++CurveIndex) { Ar << Params.name[CurveIndex]; }

	Ar << Params.m_fStrengthResponse;
	Ar << Params.m_fDirectionResponse;
	
	Ar << Params.m_fAnchorOffset;
	Ar << Params.m_fAnchorDistanceScale;
	
	for (int32 OscIndex = 0; OscIndex < FSpeedTreeWind::NUM_OSC_COMPONENTS; ++OscIndex)
	{
		SERIALIZE_CURVE(m_afFrequencies[OscIndex]);
	}
	
	Ar << Params.m_fGlobalHeight;
	Ar << Params.m_fGlobalHeightExponent;
	SERIALIZE_CURVE(m_afGlobalDistance);
	SERIALIZE_CURVE(m_afGlobalDirectionAdherence);
	
	for (int32 BranchIndex = 0; BranchIndex < FSpeedTreeWind::NUM_BRANCH_LEVELS; ++BranchIndex)
	{
		SERIALIZE_CURVE(m_asBranch[BranchIndex].m_afDistance);
		SERIALIZE_CURVE(m_asBranch[BranchIndex].m_afDirectionAdherence);
		SERIALIZE_CURVE(m_asBranch[BranchIndex].m_afWhip);
		Ar << Params.m_asBranch[BranchIndex].m_fTurbulence;
		Ar << Params.m_asBranch[BranchIndex].m_fTwitch;
		Ar << Params.m_asBranch[BranchIndex].m_fTwitchFreqScale;
	}

	if (Ar.UEVer() < VER_UE4_SPEEDTREE_WIND_V7)
	{
		float fDiscardOldRolling = 0.0f;
		Ar << fDiscardOldRolling;
		Ar << fDiscardOldRolling;
		Ar << fDiscardOldRolling;
		Ar << fDiscardOldRolling;
	}

	for (int32 LeafIndex = 0; LeafIndex < FSpeedTreeWind::NUM_LEAF_GROUPS; ++LeafIndex)
	{
		SERIALIZE_CURVE(m_asLeaf[LeafIndex].m_afRippleDistance);
		SERIALIZE_CURVE(m_asLeaf[LeafIndex].m_afTumbleFlip);
		SERIALIZE_CURVE(m_asLeaf[LeafIndex].m_afTumbleTwist);
		SERIALIZE_CURVE(m_asLeaf[LeafIndex].m_afTumbleDirectionAdherence);
		SERIALIZE_CURVE(m_asLeaf[LeafIndex].m_afTwitchThrow);
		Ar << Params.m_asLeaf[LeafIndex].m_fTwitchSharpness;
		Ar << Params.m_asLeaf[LeafIndex].m_fRollMaxScale;
		Ar << Params.m_asLeaf[LeafIndex].m_fRollMinScale;
		Ar << Params.m_asLeaf[LeafIndex].m_fRollSpeed;
		Ar << Params.m_asLeaf[LeafIndex].m_fRollSeparation;
		Ar << Params.m_asLeaf[LeafIndex].m_fLeewardScalar;
	}
	
	SERIALIZE_CURVE(m_afFrondRippleDistance);
	Ar << Params.m_fFrondRippleTile;
	Ar << Params.m_fFrondRippleLightingScalar;
	
	if (Ar.UEVer() >= VER_UE4_SPEEDTREE_WIND_V7)
	{
		Ar << Params.m_fRollingNoiseSize;
		Ar << Params.m_fRollingNoiseTwist;
		Ar << Params.m_fRollingNoiseTurbulence;
		Ar << Params.m_fRollingNoisePeriod;
		Ar << Params.m_fRollingNoiseSpeed;
		Ar << Params.m_fRollingBranchFieldMin;
		Ar << Params.m_fRollingBranchLightingAdjust;
		Ar << Params.m_fRollingBranchVerticalOffset;
		Ar << Params.m_fRollingLeafRippleMin;
		Ar << Params.m_fRollingLeafTumbleMin;
	}

	Ar << Params.m_fGustFrequency;
	Ar << Params.m_fGustStrengthMin;
	Ar << Params.m_fGustStrengthMax;
	Ar << Params.m_fGustDurationMin;
	Ar << Params.m_fGustDurationMax;
	Ar << Params.m_fGustRiseScalar;
	Ar << Params.m_fGustFallScalar;

	bool Options[FSpeedTreeWind::NUM_WIND_OPTIONS];

	#define SERIALIZE_OPTION(name) { Options[FSpeedTreeWind::name] = Wind.IsOptionEnabled(FSpeedTreeWind::name); Ar << Options[FSpeedTreeWind::name]; }
	#define SKIP_OLD_OPTION() if (Ar.UEVer() < VER_UE4_SPEEDTREE_WIND_V7) { bool bDiscard = false; Ar << bDiscard; }
	
	SERIALIZE_OPTION(GLOBAL_WIND);
	SERIALIZE_OPTION(GLOBAL_PRESERVE_SHAPE);

	SERIALIZE_OPTION(BRANCH_SIMPLE_1);
	SERIALIZE_OPTION(BRANCH_DIRECTIONAL_1);
	SERIALIZE_OPTION(BRANCH_DIRECTIONAL_FROND_1);
	SERIALIZE_OPTION(BRANCH_TURBULENCE_1);
	SERIALIZE_OPTION(BRANCH_WHIP_1);
	SKIP_OLD_OPTION();
	SERIALIZE_OPTION(BRANCH_OSC_COMPLEX_1);

	SERIALIZE_OPTION(BRANCH_SIMPLE_2);
	SERIALIZE_OPTION(BRANCH_DIRECTIONAL_2);
	SERIALIZE_OPTION(BRANCH_DIRECTIONAL_FROND_2);
	SERIALIZE_OPTION(BRANCH_TURBULENCE_2);
	SERIALIZE_OPTION(BRANCH_WHIP_2);
	SKIP_OLD_OPTION();
	SERIALIZE_OPTION(BRANCH_OSC_COMPLEX_2);

	SERIALIZE_OPTION(LEAF_RIPPLE_VERTEX_NORMAL_1);
	SERIALIZE_OPTION(LEAF_RIPPLE_COMPUTED_1);
	SERIALIZE_OPTION(LEAF_TUMBLE_1);
	SERIALIZE_OPTION(LEAF_TWITCH_1);
	SKIP_OLD_OPTION();
	SERIALIZE_OPTION(LEAF_OCCLUSION_1);

	SERIALIZE_OPTION(LEAF_RIPPLE_VERTEX_NORMAL_2);
	SERIALIZE_OPTION(LEAF_RIPPLE_COMPUTED_2);
	SERIALIZE_OPTION(LEAF_TUMBLE_2);
	SERIALIZE_OPTION(LEAF_TWITCH_2);
	SKIP_OLD_OPTION();
	SERIALIZE_OPTION(LEAF_OCCLUSION_2);

	SERIALIZE_OPTION(FROND_RIPPLE_ONE_SIDED);
	SERIALIZE_OPTION(FROND_RIPPLE_TWO_SIDED);
	SERIALIZE_OPTION(FROND_RIPPLE_ADJUST_LIGHTING);

	if (Ar.UEVer() >= VER_UE4_SPEEDTREE_WIND_V7)
	{
		SERIALIZE_OPTION(ROLLING);
	}

	FVector BranchAnchor(Wind.GetBranchAnchor()[0], Wind.GetBranchAnchor()[1], Wind.GetBranchAnchor()[2]);
	float MaxBranchLength = Wind.GetMaxBranchLength();

	Ar << BranchAnchor;
	Ar << MaxBranchLength;

	if (Ar.IsLoading())
	{
		// apply values if they were loaded
		for (int32 OptionIndex = 0; OptionIndex < FSpeedTreeWind::NUM_WIND_OPTIONS; ++OptionIndex)
		{
			Wind.SetOption((FSpeedTreeWind::EOptions)OptionIndex, Options[OptionIndex]);
		}

		Wind.SetParams(Params);
		Wind.SetTreeValues(BranchAnchor, MaxBranchLength);
	}

	#undef SERIALIZE_CURVE

	return Ar;
}
