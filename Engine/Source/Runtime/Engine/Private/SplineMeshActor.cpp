// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	SplineMeshActor.cpp: Spline mesh actor class implementation.
=============================================================================*/

#include "Engine/SplineMeshActor.h"
#include "Components/SplineMeshComponent.h"
#include "Engine/CollisionProfile.h"
#include "Logging/MessageLog.h"
#include "Misc/UObjectToken.h"
#include "Misc/MapErrors.h"
#include "Engine/StaticMesh.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SplineMeshActor)

#define LOCTEXT_NAMESPACE "SplineMeshActor"

ASplineMeshActor::ASplineMeshActor(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	SetCanBeDamaged(false);

	SplineMeshComponent = ObjectInitializer.CreateDefaultSubobject<USplineMeshComponent>(this, TEXT("SplineMeshComponent0"));
	SplineMeshComponent->SetCollisionProfileName(UCollisionProfile::BlockAll_ProfileName);
	SplineMeshComponent->Mobility = EComponentMobility::Static;
	SplineMeshComponent->SetGenerateOverlapEvents(false);
	SplineMeshComponent->bAllowSplineEditingPerInstance = true;

	RootComponent = SplineMeshComponent;
}


FString ASplineMeshActor::GetDetailedInfoInternal() const
{
	check(SplineMeshComponent != nullptr);
	return SplineMeshComponent->GetDetailedInfoInternal();
}

void ASplineMeshActor::SetMobility(EComponentMobility::Type InMobility)
{
	check(SplineMeshComponent != nullptr);
	SplineMeshComponent->SetMobility(InMobility);
}

#if WITH_EDITOR

bool ASplineMeshActor::GetReferencedContentObjects(TArray<UObject*>& Objects) const
{
	Super::GetReferencedContentObjects(Objects);

	check(SplineMeshComponent != nullptr);
	if (SplineMeshComponent->GetStaticMesh() != nullptr)
	{
		Objects.Add(SplineMeshComponent->GetStaticMesh());
	}
	return true;
}

void ASplineMeshActor::CheckForErrors()
{
	Super::CheckForErrors();

	FMessageLog MapCheck("MapCheck");

	if (SplineMeshComponent->GetStaticMesh() == nullptr)
	{
		MapCheck.Warning()
			->AddToken(FUObjectToken::Create(this))
			->AddToken(FTextToken::Create(LOCTEXT("MapCheck_Message_SplineMeshNull", "Spline mesh actor has NULL StaticMesh property")))
			->AddToken(FMapErrorToken::Create(FMapErrors::StaticMeshNull));
	}
}

#endif // WITH_EDITOR

/** Returns SplineMeshComponent subobject **/
USplineMeshComponent* ASplineMeshActor::GetSplineMeshComponent() const { return SplineMeshComponent; }

#undef LOCTEXT_NAMESPACE

