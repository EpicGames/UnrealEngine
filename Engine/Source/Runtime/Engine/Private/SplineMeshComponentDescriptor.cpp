// Copyright Epic Games, Inc. All Rights Reserved.

#include "Engine/SplineMeshComponentDescriptor.h"

#include "Algo/Transform.h"
#include "Components/SplineMeshComponent.h"
#include "Components/StaticMeshComponent.h"
#include "Concepts/StaticStructProvider.h"
#include "Engine/StaticMesh.h"
#include "Materials/MaterialInstanceDynamic.h"
#include "Materials/MaterialInterface.h"
#include "Serialization/ArchiveCrc32.h"
#include "VT/RuntimeVirtualTexture.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SplineMeshComponentDescriptor)

FSplineMeshComponentDescriptorBase::FSplineMeshComponentDescriptorBase()
{
	// Note: should not really be used - prefer using FSplineMeshComponentDescriptor & FSoftSplineMeshDescriptor instead
	InitFrom(USplineMeshComponent::StaticClass()->GetDefaultObject<USplineMeshComponent>());
}

FSplineMeshComponentDescriptorBase::FSplineMeshComponentDescriptorBase(ENoInit) {}
FSplineMeshComponentDescriptorBase::~FSplineMeshComponentDescriptorBase() = default;

FSplineMeshComponentDescriptor::FSplineMeshComponentDescriptor()
	: FSplineMeshComponentDescriptorBase(NoInit)
{
	// Make sure we have proper defaults
	InitFrom(USplineMeshComponent::StaticClass()->GetDefaultObject<USplineMeshComponent>());
}

FSplineMeshComponentDescriptor::FSplineMeshComponentDescriptor(const FSoftSplineMeshComponentDescriptor& Other)
	: FSplineMeshComponentDescriptorBase(Other)
{
	StaticMesh = Other.StaticMesh.LoadSynchronous();
	Algo::Transform(Other.OverrideMaterials, OverrideMaterials, [](TSoftObjectPtr<UMaterialInterface> Material) { return Material.LoadSynchronous(); });
	OverlayMaterial = Other.OverlayMaterial.LoadSynchronous();
	Algo::Transform(Other.RuntimeVirtualTextures, RuntimeVirtualTextures, [](TSoftObjectPtr<URuntimeVirtualTexture> RVT) { return RVT.LoadSynchronous(); });
	Hash = Other.Hash;
}

FSplineMeshComponentDescriptor::~FSplineMeshComponentDescriptor() = default;

FSoftSplineMeshComponentDescriptor::FSoftSplineMeshComponentDescriptor()
	: FSplineMeshComponentDescriptorBase(NoInit)
{
	// Make sure we have proper defaults
	InitFrom(USplineMeshComponent::StaticClass()->GetDefaultObject<USplineMeshComponent>());
}

FSoftSplineMeshComponentDescriptor::FSoftSplineMeshComponentDescriptor(const FSplineMeshComponentDescriptor& Other)
	: FSplineMeshComponentDescriptorBase(Other)
{
	StaticMesh = Other.StaticMesh;
	Algo::Transform(Other.OverrideMaterials, OverrideMaterials, [](TObjectPtr<UMaterialInterface> Material) { return Material; });
	OverlayMaterial = Other.OverlayMaterial;
	Algo::Transform(Other.RuntimeVirtualTextures, RuntimeVirtualTextures, [](TObjectPtr<URuntimeVirtualTexture> RVT) { return RVT; });
	Hash = Other.Hash;
}

FSoftSplineMeshComponentDescriptor::~FSoftSplineMeshComponentDescriptor() = default;

FSplineMeshComponentDescriptor FSplineMeshComponentDescriptor::CreateFrom(const TSubclassOf<UStaticMeshComponent>& From)
{
	FSplineMeshComponentDescriptor ComponentDescriptor;

	ComponentDescriptor.InitFrom(From->GetDefaultObject<UStaticMeshComponent>());
	ComponentDescriptor.ComputeHash();

	return ComponentDescriptor;
}

void FSplineMeshComponentDescriptorBase::InitFrom(const UStaticMeshComponent* Template, bool bInitBodyInstance)
{
	check(Template);
	bEnableDiscardOnLoad = false;

	const USplineMeshComponent* SplineTemplate = Cast<USplineMeshComponent>(Template);

	// Disregard the template class if it does not stem from a spline mesh component
	if (SplineTemplate)
	{
		ComponentClass = SplineTemplate->GetClass();
	}

	Mobility = Template->Mobility;
	VirtualTextureRenderPassType = Template->VirtualTextureRenderPassType;
	LightmapType = Template->GetLightmapType();
	LightingChannels = Template->LightingChannels;
	RayTracingGroupId = Template->RayTracingGroupId;
	RayTracingGroupCullingPriority = Template->RayTracingGroupCullingPriority;
	bHasCustomNavigableGeometry = Template->bHasCustomNavigableGeometry;
	CustomDepthStencilWriteMask = Template->CustomDepthStencilWriteMask;
	VirtualTextureCullMips = Template->VirtualTextureCullMips;
	TranslucencySortPriority = Template->TranslucencySortPriority;
	OverriddenLightMapRes = Template->OverriddenLightMapRes;
	CustomDepthStencilValue = Template->CustomDepthStencilValue;
	bCastShadow = Template->CastShadow;
	bEmissiveLightSource = Template->bEmissiveLightSource;
	bCastStaticShadow = Template->bCastStaticShadow;
	bCastDynamicShadow = Template->bCastDynamicShadow;
	bCastContactShadow = Template->bCastContactShadow;
	bCastShadowAsTwoSided = Template->bCastShadowAsTwoSided;
	bCastHiddenShadow = Template->bCastHiddenShadow;
	bAffectDynamicIndirectLighting = Template->bAffectDynamicIndirectLighting;
	bAffectDynamicIndirectLightingWhileHidden = Template->bAffectIndirectLightingWhileHidden;
	bAffectDistanceFieldLighting = Template->bAffectDistanceFieldLighting;
	bReceivesDecals = Template->bReceivesDecals;
	bOverrideLightMapRes = Template->bOverrideLightMapRes;
	bUseAsOccluder = Template->bUseAsOccluder;
	bRenderCustomDepth = Template->bRenderCustomDepth;
	bHiddenInGame = Template->bHiddenInGame;
	bIsEditorOnly = Template->bIsEditorOnly;
	bVisible = Template->GetVisibleFlag();
	bVisibleInRayTracing = Template->bVisibleInRayTracing;
	bVisibleInReflections = Template->bVisibleInReflections;
	bEvaluateWorldPositionOffset = Template->bEvaluateWorldPositionOffset;
	WorldPositionOffsetDisableDistance = Template->WorldPositionOffsetDisableDistance;
	ShadowCacheInvalidationBehavior = Template->ShadowCacheInvalidationBehavior;
	DetailMode = Template->DetailMode;
	// Determine if this must render with reversed culling based on both scale and the component property
	const bool bIsLocalToWorldDeterminantNegative = Template->GetRenderMatrix().Determinant() < 0;
	bReverseCulling = Template->bReverseCulling != bIsLocalToWorldDeterminantNegative;
	bUseDefaultCollision = Template->bUseDefaultCollision;
	bGenerateOverlapEvents = Template->GetGenerateOverlapEvents();
	bOverrideNavigationExport = Template->bOverrideNavigationExport;
	bForceNavigationObstacle = Template->bForceNavigationObstacle;
	bFillCollisionUnderneathForNavmesh = Template->bFillCollisionUnderneathForNavmesh;
	bNeverNeedsCookedCollisionData = SplineTemplate ? SplineTemplate->GetbNeverNeedsCookedCollisionData() : false;

#if WITH_EDITORONLY_DATA
	HLODBatchingPolicy = Template->HLODBatchingPolicy;
	bIncludeInHLOD = Template->bEnableAutoLODGeneration;
	bConsiderForActorPlacementWhenHidden = Template->bConsiderForActorPlacementWhenHidden;
#endif

	if (bInitBodyInstance)
	{
		BodyInstance.CopyBodyInstancePropertiesFrom(&Template->BodyInstance);
	}
}

void FSplineMeshComponentDescriptor::InitFrom(const UStaticMeshComponent* Template, bool bInitBodyInstance)
{
	StaticMesh = Template->GetStaticMesh();
	OverrideMaterials = Template->OverrideMaterials;
	OverlayMaterial = Template->OverlayMaterial;
	RuntimeVirtualTextures = Template->RuntimeVirtualTextures;

	Super::InitFrom(Template, bInitBodyInstance);
}

void FSoftSplineMeshComponentDescriptor::InitFrom(const UStaticMeshComponent* Template, bool bInitBodyInstance)
{
	StaticMesh = Template->GetStaticMesh();
	Algo::Transform(Template->OverrideMaterials, OverrideMaterials, [](TObjectPtr<UMaterialInterface> Material) { return Material; });
	OverlayMaterial = Template->OverlayMaterial;
	Algo::Transform(Template->RuntimeVirtualTextures, RuntimeVirtualTextures, [](TObjectPtr<URuntimeVirtualTexture> RVT) { return RVT; });

	Super::InitFrom(Template, bInitBodyInstance);
}

void FSplineMeshComponentDescriptorBase::PostLoadFixup(UObject* Loader)
{
	check(Loader);

	// Necessary to update the collision Response Container from the array
	BodyInstance.FixupData(Loader);
}

bool FSplineMeshComponentDescriptorBase::operator!=(const FSplineMeshComponentDescriptorBase& Other) const
{
	return !(*this == Other);
}

bool FSplineMeshComponentDescriptor::operator!=(const FSplineMeshComponentDescriptor& Other) const
{
	return !(*this == Other);
}

bool FSoftSplineMeshComponentDescriptor::operator!=(const FSoftSplineMeshComponentDescriptor& Other) const
{
	return !(*this == Other);
}

bool FSplineMeshComponentDescriptorBase::operator==(const FSplineMeshComponentDescriptorBase& Other) const
{
	return ComponentClass == Other.ComponentClass &&
	Mobility == Other.Mobility &&
	VirtualTextureRenderPassType == Other.VirtualTextureRenderPassType &&
	LightmapType == Other.LightmapType &&
	GetLightingChannelMaskForStruct(LightingChannels) == GetLightingChannelMaskForStruct(Other.LightingChannels) &&
	RayTracingGroupId == Other.RayTracingGroupId &&
	RayTracingGroupCullingPriority == Other.RayTracingGroupCullingPriority &&
	bHasCustomNavigableGeometry == Other.bHasCustomNavigableGeometry &&
	CustomDepthStencilWriteMask == Other.CustomDepthStencilWriteMask &&
	VirtualTextureCullMips == Other.VirtualTextureCullMips &&
	TranslucencySortPriority == Other.TranslucencySortPriority &&
	OverriddenLightMapRes == Other.OverriddenLightMapRes &&
	CustomDepthStencilValue == Other.CustomDepthStencilValue &&
	bCastShadow == Other.bCastShadow &&
	bEmissiveLightSource == Other.bEmissiveLightSource &&
	bCastStaticShadow == Other.bCastStaticShadow &&
	bCastDynamicShadow == Other.bCastDynamicShadow &&
	bCastContactShadow == Other.bCastContactShadow &&
	bCastShadowAsTwoSided == Other.bCastShadowAsTwoSided &&
	bCastHiddenShadow == Other.bCastHiddenShadow &&
	bAffectDynamicIndirectLighting == Other.bAffectDynamicIndirectLighting &&
	bAffectDynamicIndirectLightingWhileHidden == Other.bAffectDynamicIndirectLightingWhileHidden &&
	bAffectDistanceFieldLighting == Other.bAffectDistanceFieldLighting &&
	bReceivesDecals == Other.bReceivesDecals &&
	bOverrideLightMapRes == Other.bOverrideLightMapRes &&
	bUseAsOccluder == Other.bUseAsOccluder &&
	bRenderCustomDepth == Other.bRenderCustomDepth &&
	bEnableDiscardOnLoad == Other.bEnableDiscardOnLoad &&
	bHiddenInGame == Other.bHiddenInGame &&
	bIsEditorOnly == Other.bIsEditorOnly &&
	bVisible == Other.bVisible &&
	bVisibleInRayTracing == Other.bVisibleInRayTracing &&
	bVisibleInReflections == Other.bVisibleInReflections &&
	bEvaluateWorldPositionOffset == Other.bEvaluateWorldPositionOffset &&
	bReverseCulling == Other.bReverseCulling &&
	bUseDefaultCollision == Other.bUseDefaultCollision &&
	bGenerateOverlapEvents == Other.bGenerateOverlapEvents &&
	bOverrideNavigationExport == Other.bOverrideNavigationExport &&
	bForceNavigationObstacle == Other.bForceNavigationObstacle &&
	bFillCollisionUnderneathForNavmesh == Other.bFillCollisionUnderneathForNavmesh &&
	WorldPositionOffsetDisableDistance == Other.WorldPositionOffsetDisableDistance &&
	ShadowCacheInvalidationBehavior == Other.ShadowCacheInvalidationBehavior &&
	DetailMode == Other.DetailMode &&
	bNeverNeedsCookedCollisionData == Other.bNeverNeedsCookedCollisionData &&
#if WITH_EDITORONLY_DATA
	HLODBatchingPolicy == Other.HLODBatchingPolicy &&
	bIncludeInHLOD == Other.bIncludeInHLOD &&
	bConsiderForActorPlacementWhenHidden == Other.bConsiderForActorPlacementWhenHidden &&
#endif // WITH_EDITORONLY_DATA
	BodyInstance.GetCollisionEnabled() == Other.BodyInstance.GetCollisionEnabled() && 
	BodyInstance.GetCollisionResponse() == Other.BodyInstance.GetCollisionResponse() &&
	BodyInstance.DoesUseCollisionProfile() == Other.BodyInstance.DoesUseCollisionProfile() &&
	(!BodyInstance.DoesUseCollisionProfile() || (BodyInstance.GetCollisionProfileName() == Other.BodyInstance.GetCollisionProfileName()));
}

bool FSplineMeshComponentDescriptor::operator==(const FSplineMeshComponentDescriptor& Other) const
{
	return (Hash == 0 || Other.Hash == 0 || Hash == Other.Hash) && // Check hash first, other checks are in case of Hash collision
		StaticMesh == Other.StaticMesh &&
		OverrideMaterials == Other.OverrideMaterials &&
		OverlayMaterial == Other.OverlayMaterial &&
		RuntimeVirtualTextures == Other.RuntimeVirtualTextures &&
		Super::operator==(Other);
}

bool FSoftSplineMeshComponentDescriptor::operator==(const FSoftSplineMeshComponentDescriptor& Other) const
{
	return (Hash == 0 || Other.Hash == 0 || Hash == Other.Hash) && // Check hash first, other checks are in case of Hash collision
		StaticMesh == Other.StaticMesh &&
		OverrideMaterials == Other.OverrideMaterials &&
		OverlayMaterial == Other.OverlayMaterial &&
		RuntimeVirtualTextures == Other.RuntimeVirtualTextures &&
		Super::operator==(Other);
}

uint32 FSplineMeshComponentDescriptorBase::ComputeHash() const
{
	FArchiveCrc32 CrcArchive;

	Hash = 0; // we don't want the hash to impact the calculation
	CrcArchive << *this;
	Hash = CrcArchive.GetCrc();

	return Hash;
}

uint32 FSplineMeshComponentDescriptor::ComputeHash() const
{
	Super::ComputeHash();

	FSplineMeshComponentDescriptor& MutableSelf = *const_cast<FSplineMeshComponentDescriptor*>(this);
	FArchiveCrc32 CrcArchive(Hash);
	CrcArchive << MutableSelf.StaticMesh;
	CrcArchive << MutableSelf.OverrideMaterials;
	CrcArchive << MutableSelf.OverlayMaterial;
	CrcArchive << MutableSelf.RuntimeVirtualTextures;
	Hash = CrcArchive.GetCrc();

	return Hash;
}

uint32 FSoftSplineMeshComponentDescriptor::ComputeHash() const
{
	Super::ComputeHash();

	FSoftSplineMeshComponentDescriptor& MutableSelf = *const_cast<FSoftSplineMeshComponentDescriptor*>(this);
	FArchiveCrc32 CrcArchive(Hash);
	CrcArchive << MutableSelf.StaticMesh;
	CrcArchive << MutableSelf.OverrideMaterials;
	CrcArchive << MutableSelf.OverlayMaterial;
	CrcArchive << MutableSelf.RuntimeVirtualTextures;
	Hash = CrcArchive.GetCrc();

	return Hash;
}

USplineMeshComponent* FSplineMeshComponentDescriptorBase::CreateComponent(UObject* Outer, FName Name, EObjectFlags ObjectFlags) const
{
	USplineMeshComponent* SplineMeshComponent = NewObject<USplineMeshComponent>(Outer, ComponentClass, Name, ObjectFlags);
	
	InitComponent(SplineMeshComponent);

	return SplineMeshComponent;
}

void FSplineMeshComponentDescriptorBase::InitComponent(USplineMeshComponent* SplineMeshComponent) const
{
	SplineMeshComponent->Mobility = Mobility;
	SplineMeshComponent->VirtualTextureRenderPassType = VirtualTextureRenderPassType;
	SplineMeshComponent->SetLightmapType(LightmapType);
	SplineMeshComponent->LightingChannels = LightingChannels;
	SplineMeshComponent->RayTracingGroupId = RayTracingGroupId;
	SplineMeshComponent->RayTracingGroupCullingPriority = RayTracingGroupCullingPriority;
	SplineMeshComponent->bHasCustomNavigableGeometry = bHasCustomNavigableGeometry;
	SplineMeshComponent->CustomDepthStencilWriteMask = CustomDepthStencilWriteMask;
	SplineMeshComponent->BodyInstance.CopyBodyInstancePropertiesFrom(&BodyInstance);
	SplineMeshComponent->VirtualTextureCullMips = VirtualTextureCullMips;
	SplineMeshComponent->TranslucencySortPriority = TranslucencySortPriority;
	SplineMeshComponent->OverriddenLightMapRes = OverriddenLightMapRes;
	SplineMeshComponent->CustomDepthStencilValue = CustomDepthStencilValue;
	SplineMeshComponent->CastShadow = bCastShadow;
	SplineMeshComponent->bEmissiveLightSource = bEmissiveLightSource;
	SplineMeshComponent->bCastStaticShadow = bCastStaticShadow;
	SplineMeshComponent->bCastDynamicShadow = bCastDynamicShadow;
	SplineMeshComponent->bCastContactShadow = bCastContactShadow;
	SplineMeshComponent->bCastShadowAsTwoSided = bCastShadowAsTwoSided;
	SplineMeshComponent->bCastHiddenShadow = bCastHiddenShadow;
	SplineMeshComponent->bAffectDynamicIndirectLighting = bAffectDynamicIndirectLighting;
	SplineMeshComponent->bAffectIndirectLightingWhileHidden = bAffectDynamicIndirectLightingWhileHidden;
	SplineMeshComponent->bAffectDistanceFieldLighting = bAffectDistanceFieldLighting;
	SplineMeshComponent->bReceivesDecals = bReceivesDecals;
	SplineMeshComponent->bOverrideLightMapRes = bOverrideLightMapRes;
	SplineMeshComponent->bUseAsOccluder = bUseAsOccluder;
	SplineMeshComponent->bRenderCustomDepth = bRenderCustomDepth;
	SplineMeshComponent->bHiddenInGame = bHiddenInGame;
	SplineMeshComponent->bIsEditorOnly = bIsEditorOnly;
	SplineMeshComponent->SetVisibleFlag(bVisible);
	SplineMeshComponent->bVisibleInRayTracing = bVisibleInRayTracing;
	SplineMeshComponent->bVisibleInReflections = bVisibleInReflections;
	SplineMeshComponent->bEvaluateWorldPositionOffset = bEvaluateWorldPositionOffset;
	SplineMeshComponent->bReverseCulling = bReverseCulling;
	SplineMeshComponent->bUseDefaultCollision = bUseDefaultCollision;
	SplineMeshComponent->SetGenerateOverlapEvents(bGenerateOverlapEvents);
	SplineMeshComponent->bOverrideNavigationExport = bOverrideNavigationExport;
	SplineMeshComponent->bForceNavigationObstacle = bForceNavigationObstacle;
	SplineMeshComponent->bFillCollisionUnderneathForNavmesh = bFillCollisionUnderneathForNavmesh;
	SplineMeshComponent->WorldPositionOffsetDisableDistance = WorldPositionOffsetDisableDistance;
	SplineMeshComponent->ShadowCacheInvalidationBehavior = ShadowCacheInvalidationBehavior;
	SplineMeshComponent->DetailMode = DetailMode;
	SplineMeshComponent->SetbNeverNeedsCookedCollisionData(bNeverNeedsCookedCollisionData);
	
#if WITH_EDITORONLY_DATA
	SplineMeshComponent->HLODBatchingPolicy = HLODBatchingPolicy;
	SplineMeshComponent->bEnableAutoLODGeneration = bIncludeInHLOD;
	SplineMeshComponent->bConsiderForActorPlacementWhenHidden = bConsiderForActorPlacementWhenHidden;
#endif // WITH_EDITORONLY_DATA
}

void FSplineMeshComponentDescriptor::InitComponent(USplineMeshComponent* SplineMeshComponent) const
{
	SplineMeshComponent->SetStaticMesh(StaticMesh);

	auto GetMaterial = [SplineMeshComponent](UMaterialInterface* MaterialInterface)
	{
		if (MaterialInterface && !MaterialInterface->IsAsset())
		{
			// If the material is equivalent to its parent, just take a reference to its parent rather than making another redundant object 
			if (UMaterialInstance* Instance = Cast<UMaterialInstance>(MaterialInterface); Instance && Instance->IsRedundant())
			{
				MaterialInterface = Instance->Parent;
			}
			else
			{
				// As override materials are normally outered to their owner component, we need to duplicate them here to make sure we don't create
				// references to actors in other levels (for packed level instances or HLOD actors).
				MaterialInterface = DuplicateObject<UMaterialInterface>(MaterialInterface, SplineMeshComponent);

				// If the MID we just duplicated has a nanite override that's also not an asset, duplicate that too
				UMaterialInstanceDynamic* OverrideMID = Cast<UMaterialInstanceDynamic>(MaterialInterface);
				UMaterialInterface* NaniteOverride = OverrideMID ? OverrideMID->GetNaniteOverride() : nullptr;
				if (NaniteOverride && !NaniteOverride->IsAsset())
				{
					OverrideMID->SetNaniteOverride(DuplicateObject<UMaterialInterface>(NaniteOverride, SplineMeshComponent));
				}
			}
		}

		return MaterialInterface;
	};

	SplineMeshComponent->OverrideMaterials.Empty(OverrideMaterials.Num());
	for (UMaterialInterface* OverrideMaterial : OverrideMaterials)
	{
		SplineMeshComponent->OverrideMaterials.Add(GetMaterial(OverrideMaterial));
	}
	SplineMeshComponent->OverlayMaterial = GetMaterial(OverlayMaterial);
	SplineMeshComponent->RuntimeVirtualTextures = RuntimeVirtualTextures;

	Super::InitComponent(SplineMeshComponent);
}

void FSoftSplineMeshComponentDescriptor::InitComponent(USplineMeshComponent* SplineMeshComponent) const
{
	SplineMeshComponent->SetStaticMesh(StaticMesh.LoadSynchronous());

	auto GetMaterial = [SplineMeshComponent](const TSoftObjectPtr<UMaterialInterface>& MaterialInterfacePtr)
	{
		UMaterialInterface* MaterialInterface = MaterialInterfacePtr.LoadSynchronous();
		if (MaterialInterface && !MaterialInterface->IsAsset())
		{
			// If the material is equivalent to its parent, just take a reference to its parent rather than making another redundant object 
			if (UMaterialInstance* Instance = Cast<UMaterialInstance>(MaterialInterface); Instance && Instance->IsRedundant())
			{
				MaterialInterface = Instance->Parent;
			}
			else
			{
				// As override materials are normally outered to their owner component, we need to duplicate them here to make sure we don't create
				// references to actors in other levels (for packed level instances or HLOD actors).
				MaterialInterface = DuplicateObject<UMaterialInterface>(MaterialInterface, SplineMeshComponent);

				// If the MID we just duplicated has a nanite override that's also not an asset, duplicate that too
				UMaterialInstanceDynamic* OverrideMID = Cast<UMaterialInstanceDynamic>(MaterialInterface);
				UMaterialInterface* NaniteOverride = OverrideMID ? OverrideMID->GetNaniteOverride() : nullptr; 
				if (NaniteOverride && !NaniteOverride->IsAsset())
				{
					OverrideMID->SetNaniteOverride(DuplicateObject<UMaterialInterface>(NaniteOverride, SplineMeshComponent));
				}
			}
		}
		
		return MaterialInterface;
	};

	SplineMeshComponent->OverrideMaterials.Empty(OverrideMaterials.Num());
	for (const TSoftObjectPtr<UMaterialInterface>& OverrideMaterialPtr : OverrideMaterials)
	{
		SplineMeshComponent->OverrideMaterials.Add(GetMaterial(OverrideMaterialPtr));
	}
	SplineMeshComponent->OverlayMaterial = GetMaterial(OverlayMaterial);

	SplineMeshComponent->RuntimeVirtualTextures.Empty(RuntimeVirtualTextures.Num());
	for (const TSoftObjectPtr<URuntimeVirtualTexture>& RuntimeVirtualTexturePtr : RuntimeVirtualTextures)
	{
		if (URuntimeVirtualTexture* RuntimeVirtualTexture = RuntimeVirtualTexturePtr.LoadSynchronous())
		{
			SplineMeshComponent->RuntimeVirtualTextures.Add(RuntimeVirtualTexture);
		}
	}

	Super::InitComponent(SplineMeshComponent);
}
