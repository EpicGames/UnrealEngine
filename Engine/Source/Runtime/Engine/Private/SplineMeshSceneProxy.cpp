// Copyright Epic Games, Inc. All Rights Reserved.

#include "SplineMeshSceneProxy.h"
#include "Engine/StaticMesh.h"
#include "Materials/Material.h"
#include "Materials/MaterialRenderProxy.h"
#include "MaterialDomain.h"
#include "MeshMaterialShader.h"
#include "Misc/DelayedAutoRegister.h"
#include "StaticMeshComponentLODInfo.h"
#include "StaticMeshResources.h"
#include "RayTracingInstance.h"
#include "SceneInterface.h"
#include "SplineMeshSceneProxyDesc.h"
#include "DistanceFieldAtlas.h"
#include "MeshCardBuild.h"

IMPLEMENT_TYPE_LAYOUT(FSplineMeshVertexFactoryShaderParameters);

IMPLEMENT_GLOBAL_SHADER_PARAMETER_STRUCT(FSplineMeshVFPrimitiveShaderParameters, "SplineMeshVFPrimitiveShaderParameters");

static TAutoConsoleVariable<int32> CVarRayTracingSplineMeshes(
	TEXT("r.RayTracing.Geometry.SplineMeshes"),
	1,
	TEXT("Include splines meshes in ray tracing effects (default = 1 (spline meshes enabled in ray tracing))"));

static TAutoConsoleVariable<int32> CVarRayTracingSplineMeshesForceUpdate(
	TEXT("r.RayTracing.Geometry.SplineMeshes.ForceUpdate"),
	0,
	TEXT("Forces ray tracing geometry for spline meshes to be updated every frame"));

bool FSplineMeshVertexFactory::ShouldCompilePermutation(const FVertexFactoryShaderPermutationParameters& Parameters)
{
	return (Parameters.MaterialParameters.bIsUsedWithSplineMeshes || Parameters.MaterialParameters.bIsSpecialEngineMaterial)
		&& FLocalVertexFactory::ShouldCompilePermutation(Parameters);
}

/** Modify compile environment to enable spline deformation */
void FSplineMeshVertexFactory::ModifyCompilationEnvironment(const FVertexFactoryShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
{
	OutEnvironment.SetDefine(TEXT("VF_SUPPORTS_SPEEDTREE_WIND"), TEXT("0"));
	FLocalVertexFactory::ModifyCompilationEnvironment(Parameters, OutEnvironment);

	OutEnvironment.SetDefine(TEXT("USE_SPLINEDEFORM"), TEXT("1"));
	OutEnvironment.SetDefine(TEXT("USE_SPLINE_MESH_SCENE_RESOURCES"), UseSplineMeshSceneResources(Parameters.Platform));
}

/**
 * FSplineMeshVertexFactory does not support manual vertex fetch yet so worst case element set is returned to make sure the PSO can be compiled
 */
void FSplineMeshVertexFactory::GetPSOPrecacheVertexFetchElements(EVertexInputStreamType VertexInputStreamType, FVertexDeclarationElementList& Elements)
{
	FLocalVertexFactory::GetPSOPrecacheVertexFetchElements(VertexInputStreamType, Elements);
}

bool FSplineMeshSceneProxyCommon::OnMeshCardsTransformChanged(const FVector& LocalToWorldScale)
{
	const FVector3f NewLocalToWorldScale(LocalToWorldScale);
	const bool bWasUniform = CachedLocalToWorldScale.IsUniform();
	const bool bIsUniform = NewLocalToWorldScale.IsUniform();

	// Uniformity changed or non-uniform scale changed
	const bool bRefitCardRepresentation = bWasUniform != bIsUniform || !(bIsUniform || (NewLocalToWorldScale - CachedLocalToWorldScale).IsNearlyZero());

	CachedLocalToWorldScale = NewLocalToWorldScale;
	return bRefitCardRepresentation;
}

FMatrix FSplineMeshSceneProxyCommon::GetMeshCardsToWorld(const FMatrix& LocalToWorld) const
{
	return FTransform(FQuat(MeshCardsToLocalRotation), FVector(MeshCardsToLocalTranslation)).ToMatrixNoScale()
		* (bCardsInWorldScale ? LocalToWorld.GetMatrixWithoutScale() : LocalToWorld);
}

void FSplineMeshSceneProxyCommon::SetSplineTextureCoord_RenderThread(uint32 SplineIndex, FUintVector2 TexCoord)
{
	check(SplineIndex == 0); // currently only support one spline type, but could do instanced later
	SplineParams.TextureCoord = TexCoord;
	RepackSplineMeshParamsInternal();
}
	
void FSplineMeshSceneProxyCommon::UpdateSplineMeshParams_RenderThread(const FSplineMeshShaderParams& Params)
{
	// Don't replace the texture coord that was assigned to us in SetSplineTextureCoord
	FUintVector2 TexCoord = SplineParams.TextureCoord;
	SplineParams = Params;
	SplineParams.TextureCoord = TexCoord;
	RepackSplineMeshParamsInternal();
}

void FSplineMeshSceneProxyCommon::RepackSplineMeshParamsInternal()
{
	auto& SceneProxy = GetProxy();
		
	// NOTE: If the payload extension was not initialized, we're probably not using GPU Scene
	if (SplineMeshInstanceData.Update(SplineParams))
	{
		// Request a GPU Scene update for this primitive so it updates its instance data
		SceneProxy.GetScene().RequestGPUSceneUpdate(*SceneProxy.GetPrimitiveSceneInfo(), EPrimitiveDirtyState::ChangedOther);
	}

	if (SplineMeshVFPrimitiveShaderParametersUB)
	{
		FSplineMeshVFPrimitiveShaderParameters SplineMeshVFPrimitiveShaderParameters;
		PackSplineMeshParams(SplineParams, SplineMeshVFPrimitiveShaderParameters.SplineParams);
		SplineMeshVFPrimitiveShaderParametersUB.UpdateUniformBufferImmediate(FRHICommandListImmediate::Get(), SplineMeshVFPrimitiveShaderParameters);
	}

	bUpdateRayTracingGeometry = true;
}

FSplineMeshSceneProxy::FSplineMeshSceneProxy(USplineMeshComponent* InComponent) :
	FSplineMeshSceneProxy(FStaticMeshSceneProxyDesc(InComponent), FSplineMeshSceneProxyDesc(InComponent))
{}

FSplineMeshSceneProxy::FSplineMeshSceneProxy(const FStaticMeshSceneProxyDesc& InMeshDesc, const FSplineMeshSceneProxyDesc& InSplineDesc) :
	FStaticMeshSceneProxy(InMeshDesc, false)
{
	bSupportsDistanceFieldRepresentation = false;

	// Mark ourselves as a spline mesh
	bSplineMesh = true;

	// Dynamic draw path without Nanite isn't supported by Lumen
	bVisibleInLumenScene = false;

#if RHI_RAYTRACING
	if (bSupportRayTracing)
	{
		bHasRayTracingRepresentation = true;
		bDynamicRayTracingGeometry = true;
	}
#endif

	// make sure all the materials are okay to be rendered as a spline mesh
	for (FStaticMeshSceneProxy::FLODInfo& LODInfo : LODs)
	{
		for (FStaticMeshSceneProxy::FLODInfo::FSectionInfo& Section : LODInfo.Sections)
		{
			if (!Section.Material->CheckMaterialUsage_Concurrent(MATUSAGE_SplineMesh))
			{
				Section.Material = UMaterial::GetDefaultMaterial(MD_Surface);
			}
		}
	}

	// Copy spline params from component
	SplineParams = InSplineDesc.CalculateShaderParams();

	// If we're using GPU Scene, we place the spline mesh parameters in the instance data buffer, with the
	// exception of mobile platforms that are unable to pull this data from the structured buffer in the VS
	const ERHIFeatureLevel::Type FeatureLevel = GetScene().GetFeatureLevel();
	if (FeatureLevel > ERHIFeatureLevel::ES3_1 && UseGPUScene(GetScene().GetShaderPlatform()))
	{
		SplineMeshInstanceData.Setup(SplineParams);
		SetupInstanceSceneDataBuffers(&SplineMeshInstanceData);
	}
	else
	{
		FSplineMeshVFPrimitiveShaderParameters SplineMeshVFPrimitiveShaderParameters;
		PackSplineMeshParams(SplineParams, SplineMeshVFPrimitiveShaderParameters.SplineParams);
		SplineMeshVFPrimitiveShaderParametersUB = CreateUniformBufferImmediate(SplineMeshVFPrimitiveShaderParameters, EUniformBufferUsage::UniformBuffer_MultiFrame);
	}

	for (int32 LODIndex = 0; LODIndex < LODs.Num(); LODIndex++)
	{
		FSplineMeshSceneProxyDesc::InitVertexFactory(InMeshDesc.GetStaticMesh(), FeatureLevel, LODIndex, nullptr); // we always need this one for shadows etc
		if (InMeshDesc.LODData.IsValidIndex(LODIndex) && InMeshDesc.LODData[LODIndex].OverrideVertexColors)
		{
			FSplineMeshSceneProxyDesc::InitVertexFactory(InMeshDesc.GetStaticMesh(), FeatureLevel, LODIndex, InMeshDesc.LODData[LODIndex].OverrideVertexColors);
		}
	}

#if RHI_RAYTRACING
	if (bSupportRayTracing)
	{
		const int32 NumRayTracingLods = InMeshDesc.GetStaticMesh() && !InMeshDesc.GetStaticMesh()->GetRenderData()->RayTracingProxy->bUsingRenderingLODs ? InMeshDesc.GetStaticMesh()->GetRenderData()->RayTracingProxy->LODs.Num() : 0;
		for (int32 LODIndex = 0; LODIndex < NumRayTracingLods; LODIndex++)
		{
			FSplineMeshSceneProxyDesc::InitRayTracingProxyVertexFactory(InMeshDesc.GetStaticMesh(), FeatureLevel, LODIndex, nullptr);
		}
	}
#endif
}

SIZE_T FSplineMeshSceneProxy::GetTypeHash() const
{
	static size_t UniquePointer;
	return reinterpret_cast<size_t>(&UniquePointer);
}

void FSplineMeshSceneProxy::SetupMeshBatchForSpline(int32 InLODIndex, FMeshBatch& OutMeshBatch) const
{
	const FStaticMeshVertexFactories& VFs = RenderData->LODVertexFactories[InLODIndex];
	check(OutMeshBatch.Elements.Num() == 1);
	OutMeshBatch.VertexFactory = OutMeshBatch.Elements[0].bUserDataIsColorVertexBuffer ? VFs.SplineVertexFactoryOverrideColorVertexBuffer : VFs.SplineVertexFactory;
	check(OutMeshBatch.VertexFactory);
	OutMeshBatch.Elements[0].PrimitiveUniformBuffer = GetUniformBuffer();
	OutMeshBatch.Elements[0].LooseParametersUniformBuffer = SplineMeshVFPrimitiveShaderParametersUB;
	OutMeshBatch.ReverseCulling ^= (SplineParams.StartScale.X < 0) ^ (SplineParams.StartScale.Y < 0);
}

void FSplineMeshSceneProxy::SetupRayTracingMeshBatchForSpline(int32 InLODIndex, FMeshBatch& OutMeshBatch) const
{
	const FStaticMeshVertexFactories& VFs = (*RenderData->RayTracingProxy->LODVertexFactories)[InLODIndex];
	check(OutMeshBatch.Elements.Num() == 1);
	OutMeshBatch.VertexFactory = OutMeshBatch.Elements[0].bUserDataIsColorVertexBuffer ? VFs.SplineVertexFactoryOverrideColorVertexBuffer : VFs.SplineVertexFactory;
	check(OutMeshBatch.VertexFactory);
	OutMeshBatch.Elements[0].PrimitiveUniformBuffer = GetUniformBuffer();
	OutMeshBatch.Elements[0].LooseParametersUniformBuffer = SplineMeshVFPrimitiveShaderParametersUB;
	OutMeshBatch.ReverseCulling ^= (SplineParams.StartScale.X < 0) ^ (SplineParams.StartScale.Y < 0);
}

bool FSplineMeshSceneProxy::GetShadowMeshElement(int32 LODIndex, int32 BatchIndex, uint8 InDepthPriorityGroup, FMeshBatch& OutMeshBatch, bool bDitheredLODTransition) const
{
	//checkf(LODIndex == 0, TEXT("Getting spline static mesh element with invalid LOD [%d]"), LODIndex);

	if (FStaticMeshSceneProxy::GetShadowMeshElement(LODIndex, BatchIndex, InDepthPriorityGroup, OutMeshBatch, bDitheredLODTransition))
	{
		SetupMeshBatchForSpline(LODIndex, OutMeshBatch);
		return true;
	}
	return false;
}

bool FSplineMeshSceneProxy::GetMeshElement(int32 LODIndex, int32 BatchIndex, int32 SectionIndex, uint8 InDepthPriorityGroup, bool bUseSelectionOutline, bool bAllowPreCulledIndices, FMeshBatch& OutMeshBatch) const
{
	//checkf(LODIndex == 0 /*&& SectionIndex == 0*/, TEXT("Getting spline static mesh element with invalid params [%d, %d]"), LODIndex, SectionIndex);

	if (FStaticMeshSceneProxy::GetMeshElement(LODIndex, BatchIndex, SectionIndex, InDepthPriorityGroup, bUseSelectionOutline, bAllowPreCulledIndices, OutMeshBatch))
	{
		SetupMeshBatchForSpline(LODIndex, OutMeshBatch);
		return true;
	}
	return false;
}

bool FSplineMeshSceneProxy::GetWireframeMeshElement(int32 LODIndex, int32 BatchIndex, const FMaterialRenderProxy* WireframeRenderProxy, uint8 InDepthPriorityGroup, bool bAllowPreCulledIndices, FMeshBatch& OutMeshBatch) const
{
	//checkf(LODIndex == 0, TEXT("Getting spline static mesh element with invalid LOD [%d]"), LODIndex);

	if (FStaticMeshSceneProxy::GetWireframeMeshElement(LODIndex, BatchIndex, WireframeRenderProxy, InDepthPriorityGroup, bAllowPreCulledIndices, OutMeshBatch))
	{
		SetupMeshBatchForSpline(LODIndex, OutMeshBatch);
		return true;
	}
	return false;
}


bool FSplineMeshSceneProxy::GetCollisionMeshElement(int32 LODIndex, int32 BatchIndex, int32 SectionIndex, uint8 InDepthPriorityGroup, const FMaterialRenderProxy* RenderProxy, FMeshBatch& OutMeshBatch) const
{
	//checkf(LODIndex == 0, TEXT("Getting spline static mesh element with invalid LOD [%d]"), LODIndex);

	if (FStaticMeshSceneProxy::GetCollisionMeshElement(LODIndex, BatchIndex, SectionIndex, InDepthPriorityGroup, RenderProxy, OutMeshBatch))
	{
		SetupMeshBatchForSpline(LODIndex, OutMeshBatch);
		return true;
	}
	return false;
}

bool FSplineMeshSceneProxy::GetRayTracingMeshElement(int32 LODIndex, int32 BatchIndex, int32 SectionIndex, uint8 InDepthPriorityGroup, FMeshBatch& OutMeshBatch) const
{
	//checkf(LODIndex == 0 /*&& SectionIndex == 0*/, TEXT("Getting spline static mesh element with invalid params [%d, %d]"), LODIndex, SectionIndex);

	if (FStaticMeshSceneProxy::GetRayTracingMeshElement(LODIndex, BatchIndex, SectionIndex, InDepthPriorityGroup, OutMeshBatch))
	{
		SetupRayTracingMeshBatchForSpline(LODIndex, OutMeshBatch);
		return true;
	}
	return false;
}

#if RHI_RAYTRACING
void FSplineMeshSceneProxy::GetDynamicRayTracingInstances(FRayTracingInstanceCollector& Collector)
{
	if (CVarRayTracingSplineMeshes.GetValueOnRenderThread() == 0  || !bSupportRayTracing)
	{
		// TODO: Exclude proxy during ray tracing primitive gather instead of doing this early out here.
		return;
	}

	checkf(!DynamicRayTracingGeometries.IsEmpty(), TEXT("DynamicRayTracingGeometries has not been initialized correctly"));

	ESceneDepthPriorityGroup PrimitiveDPG = GetStaticDepthPriorityGroup();

	FStaticMeshRayTracingProxyLODArray& RayTracingLODs = RenderData->RayTracingProxy->LODs;

	const int32 NumLODs = RayTracingLODs.Num();

	TConstArrayView<const FSceneView*> Views = Collector.GetViews();
	const uint32 VisibilityMap = Collector.GetVisibilityMap();

	// RT geometry will be generated based on first active view and then reused for all other views
	// TODO: Expose a way for developers to control whether to reuse RT geometry or create one per-view
	const int32 FirstActiveViewIndex = FMath::CountTrailingZeros(VisibilityMap);
	checkf(Views.IsValidIndex(FirstActiveViewIndex), TEXT("There should be at least one active view when calling GetDynamicRayTracingInstances(...)."));

	const FSceneView* FirstActiveView = Views[FirstActiveViewIndex];

	int32 RayTracingMinLOD = 0;

	if (RenderData->RayTracingProxy->bUsingRenderingLODs)
	{
		RayTracingMinLOD = NumLODs - 1;

		for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ViewIndex++)
		{
			if ((VisibilityMap & (1 << ViewIndex)) == 0)
			{
				continue;
			}

			const FSceneView* View = Views[ViewIndex];

			// we use same LOD for all views
			RayTracingMinLOD = FMath::Min(RayTracingMinLOD, GetLOD(View));
		}

		// when using rendering LODs also need to limit based on currently loaded LOD
		RayTracingMinLOD = FMath::Max(RayTracingMinLOD, (int32)GetCurrentFirstLODIdx_RenderThread());
	}

	int32 LODIndex = RayTracingMinLOD;

	if (!RenderData->RayTracingProxy->bUsingRenderingLODs)
	{
		// when using WPO, need to mark the geometry group as referenced since VB/IB need to be streamed-in 
		// TODO: Support streaming only buffers when using dynamic geometry
		Collector.AddReferencedGeometryGroupForDynamicUpdate(RenderData->RayTracingGeometryGroupHandle);

		// select first LOD with valid VB/IB
		for (; LODIndex < NumLODs; LODIndex++)
		{
			FStaticMeshRayTracingProxyLOD& CurrentRayTracingLOD = RayTracingLODs[LODIndex];

			if (CurrentRayTracingLOD.bBuffersInlined || CurrentRayTracingLOD.AreBuffersStreamedIn())
			{
				break;
			}
		}
	}

	// TODO: Support streaming only buffers when using dynamic geometry
	Collector.AddReferencedGeometryGroup(RenderData->RayTracingGeometryGroupHandle);

	if (LODIndex == INDEX_NONE || LODIndex >= NumLODs)
	{
		// if none of the LODs have buffers ready for dynamic BLAS update, don't include primitive in ray tracing scene
		return;
	}

	const FStaticMeshRayTracingProxyLOD& RayTracingLOD = RayTracingLODs[LODIndex];

	FRayTracingGeometry& Geometry = DynamicRayTracingGeometries[LODIndex];

	FRayTracingInstance RayTracingInstance;

	const int32 NumBatches = GetNumMeshBatches();
	const int32 NumRayTracingMaterialEntries = RayTracingLOD.Sections->Num() * NumBatches;

	if (NumRayTracingMaterialEntries != CachedRayTracingMaterials.Num() || CachedRayTracingMaterialsLODIndex != LODIndex)
	{
		CachedRayTracingMaterials.Reset();
		CachedRayTracingMaterials.Reserve(NumRayTracingMaterialEntries);

		for (int32 BatchIndex = 0; BatchIndex < NumBatches; BatchIndex++)
		{
			for (int32 SectionIndex = 0; SectionIndex < RayTracingLOD.Sections->Num(); SectionIndex++)
			{
				FMeshBatch MeshBatch;

				bool bResult;
				if (RenderData->RayTracingProxy->bUsingRenderingLODs)
				{
					// when using rendering LODs we can reuse the main GetMeshElement(...)
					bResult = GetMeshElement(LODIndex, BatchIndex, SectionIndex, PrimitiveDPG, false, false, MeshBatch);
				}
				else
				{
					// otherwise initialize MeshBatch using ray tracing proxy VB/IB/Section data
					bResult = GetRayTracingMeshElement(LODIndex, BatchIndex, SectionIndex, PrimitiveDPG, MeshBatch);
				}

				if (bResult)
				{
					MeshBatch.ReverseCulling = bReverseCulling;
					MeshBatch.SegmentIndex = SectionIndex;
					MeshBatch.MeshIdInPrimitive = SectionIndex;

					CachedRayTracingMaterials.Add(MoveTemp(MeshBatch));
				}
			}
		}

		CachedRayTracingMaterialsLODIndex = LODIndex;
	}
	else
	{
		RayTracingInstance.bInstanceMaskAndFlagsDirty = false;
	}

	RayTracingInstance.Geometry = &Geometry;
	// scene proxies live for the duration of Render(), making array views below safe
	const FMatrix& ThisLocalToWorld = GetLocalToWorld();
	RayTracingInstance.InstanceTransformsView = MakeArrayView(&ThisLocalToWorld, 1);
	RayTracingInstance.MaterialsView = MakeArrayView(CachedRayTracingMaterials);

	const bool bAlwaysUpdate = bAnyMaterialHasWorldPositionOffset || (CVarRayTracingSplineMeshesForceUpdate.GetValueOnRenderThread() != 0);
	const bool bNeedsUpdate = bAlwaysUpdate
		|| bUpdateRayTracingGeometry
		|| (!bAlwaysUpdate && Geometry.DynamicGeometrySharedBufferGenerationID != FRayTracingGeometry::NonSharedVertexBuffers) // was using shared VB but won't use it anymore so update once
		|| !Geometry.IsValid()
		|| Geometry.IsEvicted()
		|| Geometry.GetRequiresBuild();

	FRWBuffer* VertexBuffer = &RayTracingDynamicVertexBuffer;

	if (bAlwaysUpdate)
	{
		// if updating every frame release memory and use shared VB
		VertexBuffer->Release();
		VertexBuffer = nullptr;
	}

	if (bNeedsUpdate && (*RenderData->RayTracingProxy->LODVertexFactories)[LODIndex].SplineVertexFactory->GetType()->SupportsRayTracingDynamicGeometry())
	{
		const uint32 NumVertices = RayTracingLOD.VertexBuffers->PositionVertexBuffer.GetNumVertices();

		FRayTracingDynamicGeometryUpdateParams UpdateParams;
		UpdateParams.MeshBatchesView = CachedRayTracingMaterials;
		UpdateParams.NumVertices = NumVertices;
		UpdateParams.VertexBufferSize = uint32((size_t)NumVertices * sizeof(FVector3f));
		UpdateParams.NumTriangles = Geometry.GetInitializer().TotalPrimitiveCount;
		UpdateParams.Geometry = &Geometry;
		UpdateParams.Buffer = VertexBuffer;

		Collector.AddRayTracingGeometryUpdate(FirstActiveViewIndex, MoveTemp(UpdateParams));

		bUpdateRayTracingGeometry = false;
	}

	check(CachedRayTracingMaterials.Num() == RayTracingInstance.GetMaterials().Num());
	checkf(RayTracingInstance.Geometry->GetInitializer().Segments.Num() >= CachedRayTracingMaterials.Num(), TEXT("Segments/Materials mismatch. Number of segments: %d. Number of Materials: %d. LOD Index: %d"),
		RayTracingInstance.Geometry->GetInitializer().Segments.Num(),
		CachedRayTracingMaterials.Num(),
		LODIndex);

	for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ViewIndex++)
	{
		if ((VisibilityMap & (1 << ViewIndex)) == 0)
		{
			continue;
		}

		Collector.AddRayTracingInstance(ViewIndex, RayTracingInstance);
	}
}
#endif // RHI_RAYTRACING


void FSplineMeshSceneProxy::OnTransformChanged(FRHICommandListBase& RHICmdList)
{
	// Call parent implementation
	FStaticMeshSceneProxy::OnTransformChanged(RHICmdList);

	// NOTE: The proxy's local bounds have already been padded for WPO/Displacement
	SplineMeshInstanceData.UpdateDefaultInstance(GetLocalToWorld(), GetLocalBounds());
}

void FSplineMeshSceneInstanceDataBuffers::Setup(const FSplineMeshShaderParams& InSplineMeshShaderParams)
{
	InstancePayloadExtension.SetNumUninitialized(SPLINE_MESH_PARAMS_FLOAT4_SIZE);
	Flags.bHasPerInstancePayloadExtension = true;
	Update(InSplineMeshShaderParams);
}

bool FSplineMeshSceneInstanceDataBuffers::Update(const FSplineMeshShaderParams& InSplineMeshShaderParams)
{
	if (!InstancePayloadExtension.IsEmpty())
	{
		PackSplineMeshParams(InSplineMeshShaderParams, InstancePayloadExtension);
		return true;
	}
	return false;
}

FNaniteSplineMeshSceneProxy::FNaniteSplineMeshSceneProxy(const Nanite::FMaterialAudit& NaniteMaterials, USplineMeshComponent* InComponent) :
	FNaniteSplineMeshSceneProxy(NaniteMaterials, FStaticMeshSceneProxyDesc(InComponent), FSplineMeshSceneProxyDesc(InComponent))
{}

FNaniteSplineMeshSceneProxy::FNaniteSplineMeshSceneProxy(const Nanite::FMaterialAudit& NaniteMaterials, const FStaticMeshSceneProxyDesc& InMeshDesc, const FSplineMeshSceneProxyDesc& InSplineDesc) :
	Nanite::FSceneProxy(NaniteMaterials, InMeshDesc)
{
	bSupportsDistanceFieldRepresentation = false;
	
	// Mark ourselves as a spline mesh
	bSplineMesh = true;

	// Make sure all the materials are okay to be rendered as a spline mesh or reset them
 	bool bAnyReset = false;
	for (auto& Section : GetMaterialSections())
	{
		if (Section.ShadingMaterialProxy)
		{
			UMaterialInterface* ShadingMaterial = Section.ShadingMaterialProxy->GetMaterialInterface();
			if (!ShadingMaterial->CheckMaterialUsage_Concurrent(MATUSAGE_SplineMesh))
			{
				Section.ResetToDefaultMaterial();
				bAnyReset = true;
			}
		}
	}

	if (bAnyReset)
	{
		// Update any data that is dependent upon shading materials
		OnMaterialsUpdated();
	}

	// Copy spline params from component
	SplineParams = InSplineDesc.CalculateShaderParams();
	SplineMeshInstanceData.Setup(SplineParams);
	SetupInstanceSceneDataBuffers(&SplineMeshInstanceData);

#if RHI_RAYTRACING
	if (bSupportRayTracing)
	{
		bHasRayTracingRepresentation = true;
		bDynamicRayTracingGeometry = true;

		// We only need to init vertex factories for Nanite spline meshes if they can be ray traced
		const ERHIFeatureLevel::Type FeatureLevel = GetScene().GetFeatureLevel();
		for (int32 LODIndex = 0; LODIndex < RenderData->LODResources.Num(); LODIndex++)
		{
			FSplineMeshSceneProxyDesc::InitVertexFactory(InMeshDesc.GetStaticMesh(), FeatureLevel, LODIndex, nullptr); // we always need this one for shadows etc
			if (InMeshDesc.LODData.IsValidIndex(LODIndex) && InMeshDesc.LODData[LODIndex].OverrideVertexColors)
			{
				FSplineMeshSceneProxyDesc::InitVertexFactory(InMeshDesc.GetStaticMesh(), FeatureLevel, LODIndex, InMeshDesc.LODData[LODIndex].OverrideVertexColors);
			}
		}

		const int32 NumRayTracingLods = InMeshDesc.GetStaticMesh() && !InMeshDesc.GetStaticMesh()->GetRenderData()->RayTracingProxy->bUsingRenderingLODs ? InMeshDesc.GetStaticMesh()->GetRenderData()->RayTracingProxy->LODs.Num() : 0;
		for (int32 LODIndex = 0; LODIndex < NumRayTracingLods; LODIndex++)
		{
			FSplineMeshSceneProxyDesc::InitRayTracingProxyVertexFactory(InMeshDesc.GetStaticMesh(), FeatureLevel, LODIndex, nullptr);
		}
	}
#endif
}

SIZE_T FNaniteSplineMeshSceneProxy::GetTypeHash() const
{
	static size_t UniquePointer;
	return reinterpret_cast<size_t>(&UniquePointer);
}

void FNaniteSplineMeshSceneProxy::OnTransformChanged(FRHICommandListBase& RHICmdList)
{
	// Call Nanite parent implementation
	Nanite::FSceneProxy::OnTransformChanged(RHICmdList);

	// NOTE: The proxy's local bounds have already been padded for WPO/Displacement
	SplineMeshInstanceData.UpdateDefaultInstance(GetLocalToWorld(), GetLocalBounds());

	if (IsVisibleInLumenScene() && CardRepresentationData)
	{
		const FVector LocalToWorldScale = GetLocalToWorld().GetScaleVector();
		const bool bRefitCardRepresentation = OnMeshCardsTransformChanged(LocalToWorldScale);

		if (bRefitCardRepresentation || !bHasRuntimeCardRepresentationData)
		{
			if (bHasRuntimeCardRepresentationData)
			{
				delete CardRepresentationData;
				CardRepresentationData = RenderData->LODResources[0].CardRepresentationData;
			}

			FSplineMeshSceneProxyDesc SplineDesc;
			SplineDesc.InitializeFrom(GetSplineMeshParams(), StaticMesh, RenderData);

			FBox MeshCardsBounds;
			FTransform MeshCardsToLocal;
			bCardsInWorldScale = SplineDesc.ComputeDistortedBoundsForLumenCardCapture(LocalToWorldScale, MeshCardsBounds, MeshCardsToLocal);

			FVector LocalBoundsExtent = GetLocalBounds().BoxExtent;
			if (bCardsInWorldScale)
			{
				LocalBoundsExtent *= LocalToWorldScale;
			}
			
			auto GetBoxSurfaceArea = [](const FVector& Extent)
			{
				return Extent.X * Extent.Y + Extent.Y * Extent.Z + Extent.Z * Extent.X;
			};

			FCardRepresentationData* NewCardRepresentationData;
			if (GetBoxSurfaceArea(LocalBoundsExtent) < GetBoxSurfaceArea(MeshCardsBounds.GetExtent()))
			{
				MeshCardsToLocalRotation = FQuat4f::Identity;
				MeshCardsToLocalTranslation = FVector3f::ZeroVector;
				bCardsInWorldScale = false;

				NewCardRepresentationData = new FCardRepresentationData;
				NewCardRepresentationData->MeshCardsBuildData.Bounds = GetLocalBounds().GetBox();
				NewCardRepresentationData->MeshCardsBuildData.bMostlyTwoSided = CardRepresentationData->MeshCardsBuildData.bMostlyTwoSided;

				MeshCardRepresentation::SetCardsFromBounds(NewCardRepresentationData->MeshCardsBuildData);
			}
			else
			{
				MeshCardsToLocalRotation = FQuat4f(MeshCardsToLocal.GetRotation());
				MeshCardsToLocalTranslation = FVector3f(MeshCardsToLocal.GetTranslation());

				// Mesh cards bounds is the SDF bounds or the render data bounds (see FMeshUtilities::GenerateCardRepresentationData)
				// while the new bounds is computed based on SourceMeshBounds which is the LOD0 bounds.
				// Try to match by expanding the new bounds by the same ratio.
				FVector ExpandedBoxExtent;
				if (DistanceFieldData && DistanceFieldData->LocalSpaceMeshBounds.IsValid)
				{
					ExpandedBoxExtent = FVector(DistanceFieldData->LocalSpaceMeshBounds.GetExtent());
				}
				else
				{
					ExpandedBoxExtent = RenderData->Bounds.BoxExtent;
				}

				const FVector ExpansionScale = FVector::Max(ExpandedBoxExtent / FVector::Max(SplineDesc.SourceMeshBounds.BoxExtent, FVector::OneVector), FVector::OneVector);
				const FVector MeshCardsCenter = MeshCardsBounds.GetCenter();
				const FVector MeshCardsExtent = MeshCardsBounds.GetExtent() * ExpansionScale;
				MeshCardsBounds = FBox(MeshCardsCenter - MeshCardsExtent, MeshCardsCenter + MeshCardsExtent);
			
				NewCardRepresentationData = new FCardRepresentationData;
				NewCardRepresentationData->MeshCardsBuildData = CardRepresentationData->MeshCardsBuildData;
				NewCardRepresentationData->MeshCardsBuildData.FitToNewBounds(MeshCardsBounds);
			}

			CardRepresentationData = NewCardRepresentationData;
			bHasRuntimeCardRepresentationData = true;
		}
	}
}

FMatrix FNaniteSplineMeshSceneProxy::GetMeshCardsToWorld(int32 InstanceIndex) const
{
	return FSplineMeshSceneProxyCommon::GetMeshCardsToWorld(GetLocalToWorld());
}

#if RHI_RAYTRACING

void FNaniteSplineMeshSceneProxy::GetDynamicRayTracingInstances(FRayTracingInstanceCollector& Collector)
{
	if (CVarRayTracingSplineMeshes.GetValueOnRenderThread() == 0)
	{
		return;
	}

	const bool bAlwaysUpdate = CombinedMaterialRelevance.bUsesWorldPositionOffset || (CVarRayTracingSplineMeshesForceUpdate.GetValueOnRenderThread() != 0);

	FRWBuffer* DynamicVertexBuffer = &RayTracingDynamicVertexBuffer;
	if (bAlwaysUpdate)
	{
		// if updating every frame release memory and use shared VB
		DynamicVertexBuffer->Release();
		DynamicVertexBuffer = nullptr;
	}	

	bUpdateRayTracingGeometry |= bAlwaysUpdate;

	Nanite::FSceneProxy::GetDynamicRayTracingInstances_Internal(Collector, DynamicVertexBuffer, bUpdateRayTracingGeometry);

	bUpdateRayTracingGeometry = false;
}

void FNaniteSplineMeshSceneProxy::SetupFallbackRayTracingMaterials(int32 LODIndex, TArray<FMeshBatch>& OutMaterials) const
{
	Nanite::FSceneProxy::SetupFallbackRayTracingMaterials(LODIndex, OutMaterials);

	// set up the vertex factories
	const FStaticMeshVertexFactories& VFs = (*RenderData->RayTracingProxy->LODVertexFactories)[LODIndex];
	for (auto& MeshBatch : OutMaterials)
	{
		MeshBatch.VertexFactory = MeshBatch.Elements[0].bUserDataIsColorVertexBuffer ? VFs.SplineVertexFactoryOverrideColorVertexBuffer : VFs.SplineVertexFactory;
		check(MeshBatch.VertexFactory);
		MeshBatch.ReverseCulling ^= (SplineParams.StartScale.X < 0) ^ (SplineParams.StartScale.Y < 0);
	}
}

#endif // RHI_RAYTRACING


void UpdateSplineMeshParams_RenderThread(FPrimitiveSceneProxy* SceneProxy, const FSplineMeshShaderParams& Params)
{
	check(SceneProxy->IsSplineMesh());

	if (SceneProxy->IsNaniteMesh())
	{
		static_cast<FNaniteSplineMeshSceneProxy*>(SceneProxy)->UpdateSplineMeshParams_RenderThread(Params);
	}
	else
	{
		static_cast<FSplineMeshSceneProxy*>(SceneProxy)->UpdateSplineMeshParams_RenderThread(Params);
	}
}