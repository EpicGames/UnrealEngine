// Copyright Epic Games, Inc. All Rights Reserved.

#include "SplineMeshSceneProxyDesc.h"

#include "SplineMeshSceneProxy.h"
#include "StaticMeshResources.h"
#include "Components/SplineMeshComponent.h"
#include "Engine/StaticMesh.h"
#include "HAL/ConsoleManager.h"

static TAutoConsoleVariable<bool> CVarSplineMeshFitToSourceMeshBounds(
	TEXT("r.SplineMesh.FitToSourceMeshBounds"),
	true,
	TEXT("When true, will use the bounds of the LOD 0 source mesh to fit the mesh to the spline, as opposed to the collective ")
	TEXT("mesh bounds of all LODs. This prevents gaps that might occur due bounds being expanded by lower LODs or bounds extensions.")
);

static TAutoConsoleVariable<bool> CVarSkipUpdatingMeshDeformScaleForNonNaniteMeshes(
	TEXT("r.SplineMesh.SkipUpdatingMeshDeformScaleForNonNaniteMeshes"),
	true,
	TEXT("When true, FSplineMeshShaderParams::MeshDeformScaleMinMax will only be computed if the static mesh is nanite enabled. Otherwise it's defaulted to (1,1).")
	TEXT("This speeds up CalculateShaderParams() for non-nanite meshes considerably.")
);

static FVector3f SplineEvalPos(const FVector3f& StartPos, const FVector3f& StartTangent, const FVector3f& EndPos, const FVector3f& EndTangent, float A)
{
	const float A2 = A * A;
	const float A3 = A2 * A;

	return (((2 * A3) - (3 * A2) + 1) * StartPos) + ((A3 - (2 * A2) + A) * StartTangent) + ((A3 - A2) * EndTangent) + (((-2 * A3) + (3 * A2)) * EndPos);
}

static FVector3f SplineEvalPos(const FSplineMeshParams& Params, float A)
{
	// TODO: these don't need to be doubles!
	const FVector3f StartPos = FVector3f(Params.StartPos);
	const FVector3f StartTangent = FVector3f(Params.StartTangent);
	const FVector3f EndPos = FVector3f(Params.EndPos);
	const FVector3f EndTangent = FVector3f(Params.EndTangent);

	return SplineEvalPos(StartPos, StartTangent, EndPos, EndTangent, A);
}

static FVector3f SplineEvalTangent(const FVector3f& StartPos, const FVector3f& StartTangent, const FVector3f& EndPos, const FVector3f& EndTangent, const float A)
{
	const FVector3f C = (6 * StartPos) + (3 * StartTangent) + (3 * EndTangent) - (6 * EndPos);
	const FVector3f D = (-6 * StartPos) - (4 * StartTangent) - (2 * EndTangent) + (6 * EndPos);
	const FVector3f E = StartTangent;

	const float A2 = A * A;

	return (C * A2) + (D * A) + E;
}

static FVector3f SplineEvalTangent(const FSplineMeshParams& Params, const float A)
{
	// TODO: these don't need to be doubles!
	const FVector3f StartPos = FVector3f(Params.StartPos);
	const FVector3f StartTangent = FVector3f(Params.StartTangent);
	const FVector3f EndPos = FVector3f(Params.EndPos);
	const FVector3f EndTangent = FVector3f(Params.EndTangent);

	return SplineEvalTangent(StartPos, StartTangent, EndPos, EndTangent, A);
}

static FVector3f SplineEvalDir(const FSplineMeshParams& Params, const float A)
{
	return SplineEvalTangent(Params, A).GetSafeNormal();
}

/**
* Functions used for transforming a static mesh component based on a spline.
* This needs to be updated if the spline functionality changes!
*/
static float SmoothStep(float A, float B, float X)
{
	if (X < A)
	{
		return 0.0f;
	}
	else if (X >= B)
	{
		return 1.0f;
	}
	const float InterpFraction = (X - A) / (B - A);
	return InterpFraction * InterpFraction * (3.0f - 2.0f * InterpFraction);
}

static FTransform CalcSliceTransformAtSplineOffset(const FSplineMeshParams& SplineParams, const FVector& SplineUpDir, ESplineMeshAxis::Type ForwardAxisType, bool bSmoothInterpRollScale, float Alpha, float MinT, float MaxT)
{
	// Apply hermite interp to Alpha if desired
	const float HermiteAlpha = bSmoothInterpRollScale ? SmoothStep(0.0, 1.0, Alpha) : Alpha;

	// Then find the point and direction of the spline at this point along
	FVector3f SplinePos;
	FVector3f SplineDir;

	// Use linear extrapolation
	if (Alpha < MinT)
	{
		const FVector3f StartTangent(SplineEvalTangent(SplineParams, MinT));
		SplinePos = SplineEvalPos(SplineParams, MinT) + (StartTangent * (Alpha - MinT));
		SplineDir = StartTangent.GetSafeNormal();
	}
	else if (Alpha > MaxT)
	{
		const FVector3f EndTangent(SplineEvalTangent(SplineParams, MaxT));
		SplinePos = SplineEvalPos(SplineParams, MaxT) + (EndTangent * (Alpha - MaxT));
		SplineDir = EndTangent.GetSafeNormal();
	}
	else
	{
		SplinePos = SplineEvalPos(SplineParams, Alpha);
		SplineDir = SplineEvalDir(SplineParams, Alpha);
	}

	// Find base frenet frame
	const FVector3f BaseXVec = (FVector3f(SplineUpDir) ^ SplineDir).GetSafeNormal();
	const FVector3f BaseYVec = (FVector3f(SplineDir) ^ BaseXVec).GetSafeNormal();

	// Offset the spline by the desired amount
	const FVector2f SliceOffset = FMath::Lerp(FVector2f(SplineParams.StartOffset), FVector2f(SplineParams.EndOffset), HermiteAlpha);
	SplinePos += SliceOffset.X * BaseXVec;
	SplinePos += SliceOffset.Y * BaseYVec;

	// Apply roll to frame around spline
	const float UseRoll = FMath::Lerp(SplineParams.StartRoll, SplineParams.EndRoll, HermiteAlpha);
	const float CosAng = FMath::Cos(UseRoll);
	const float SinAng = FMath::Sin(UseRoll);
	const FVector3f XVec = (CosAng * BaseXVec) - (SinAng * BaseYVec);
	const FVector3f YVec = (CosAng * BaseYVec) + (SinAng * BaseXVec);

	// Find scale at this point along spline
	const FVector2f UseScale = FMath::Lerp(FVector2f(SplineParams.StartScale), FVector2f(SplineParams.EndScale), HermiteAlpha);

	// Build overall transform
	FTransform SliceTransform;

	switch (ForwardAxisType)
	{
	case ESplineMeshAxis::X:
		SliceTransform = FTransform(FVector(SplineDir), FVector(XVec), FVector(YVec), FVector(SplinePos));
		SliceTransform.SetScale3D(FVector(1, UseScale.X, UseScale.Y));
		break;
	case ESplineMeshAxis::Y:
		SliceTransform = FTransform(FVector(YVec), FVector(SplineDir), FVector(XVec), FVector(SplinePos));
		SliceTransform.SetScale3D(FVector(UseScale.Y, 1, UseScale.X));
		break;
	case ESplineMeshAxis::Z:
		SliceTransform = FTransform(FVector(XVec), FVector(YVec), FVector(SplineDir), FVector(SplinePos));
		SliceTransform.SetScale3D(FVector(UseScale.X, UseScale.Y, 1));
		break;
	default:
		checkNoEntry();
		break;
	}

	return SliceTransform;
}

static FBoxSphereBounds GetSourceMeshBounds(const UStaticMesh* StaticMesh, const FStaticMeshRenderData* RenderData)
{
	FBoxSphereBounds Out(ForceInit);
	if (StaticMesh)
	{
		if (CVarSplineMeshFitToSourceMeshBounds.GetValueOnAnyThread() && RenderData && !RenderData->LODResources.IsEmpty())
		{
			Out = RenderData->LODResources[0].SourceMeshBounds;
		}
		else
		{
			Out = StaticMesh->GetBounds(); // legacy behavior
		}
	}
	return Out;
}


FSplineMeshSceneProxyDesc::FSplineMeshSceneProxyDesc(const USplineMeshComponent* InComponent)
{
	InitializeFrom(InComponent);
}

void FSplineMeshSceneProxyDesc::InitializeFrom(const USplineMeshComponent* InComponent)
{
	SplineParams = InComponent->SplineParams;
	// SplineUpDir is encoded as snorm16 in shader parameters so clamp to [-1, 1] to match the shader
	SplineUpDir = ClampVector(InComponent->SplineUpDir, -FVector::OneVector, FVector::OneVector);
	SplineBoundaryMin = InComponent->SplineBoundaryMin;
	SplineBoundaryMax = InComponent->SplineBoundaryMax;
	bSmoothInterpRollScale = InComponent->bSmoothInterpRollScale;
	ForwardAxis = InComponent->ForwardAxis;

	if (const UStaticMesh* StaticMesh = InComponent->GetStaticMesh())
	{
		bNaniteEnabled = StaticMesh->HasValidNaniteData();
		SourceMeshBounds = GetSourceMeshBounds(StaticMesh, StaticMesh->GetRenderData());
	}
}

void FSplineMeshSceneProxyDesc::InitializeFrom(const FSplineMeshShaderParams& ShaderParams, const UStaticMesh* StaticMesh, const FStaticMeshRenderData* RenderData)
{
	SplineParams.StartPos = FVector(ShaderParams.StartPos);
	SplineParams.EndPos = FVector(ShaderParams.EndPos);
	SplineParams.StartTangent = FVector(ShaderParams.StartTangent);
	SplineParams.EndTangent = FVector(ShaderParams.EndTangent);
	SplineParams.StartScale = FVector2D(ShaderParams.StartScale);
	SplineParams.EndScale = FVector2D(ShaderParams.EndScale);
	SplineParams.StartOffset = FVector2D(ShaderParams.StartOffset);
	SplineParams.EndOffset = FVector2D(ShaderParams.EndOffset);
	SplineParams.StartRoll = ShaderParams.StartRoll;
	SplineParams.EndRoll = ShaderParams.EndRoll;
	SplineParams.NaniteClusterBoundsScale = ShaderParams.NaniteClusterBoundsScale;

	SplineUpDir = FVector(ShaderParams.SplineUpDir);
	
	for (int32 AxisIndex = 0; AxisIndex < 3; ++AxisIndex)
	{
		if (ShaderParams.MeshDir[AxisIndex])
		{
			ForwardAxis = (ESplineMeshAxis::Type)AxisIndex;
			break;
		}
	}

	bSmoothInterpRollScale = ShaderParams.bSmoothInterpRollScale;
	bNaniteEnabled = RenderData ? RenderData->HasValidNaniteData() : false;
	
	if (StaticMesh)
	{
		SourceMeshBounds = GetSourceMeshBounds(StaticMesh, RenderData);
	}

	if (SourceMeshBounds.SphereRadius > 0.0f)
	{
		constexpr float MeshTexelLen = float(SPLINE_MESH_TEXEL_WIDTH - 1);

		if (ShaderParams.SplineDistToTexelScale != MeshTexelLen || ShaderParams.SplineDistToTexelOffset != 0.0f)
		{
			const float BoundaryLen = 1.0f / ShaderParams.MeshZScale;
			SplineBoundaryMin = -ShaderParams.MeshZOffset * BoundaryLen;
			SplineBoundaryMax = SplineBoundaryMin + BoundaryLen;
		}
	}
}

FSplineMeshShaderParams FSplineMeshSceneProxyDesc::CalculateShaderParams() const
{
	FSplineMeshShaderParams Output;

	Output.StartPos 				= FVector3f(SplineParams.StartPos);
	Output.EndPos 					= FVector3f(SplineParams.EndPos);
	Output.StartTangent 			= FVector3f(SplineParams.StartTangent);
	Output.EndTangent 				= FVector3f(SplineParams.EndTangent);
	Output.StartScale 				= FVector2f(SplineParams.StartScale);
	Output.EndScale 				= FVector2f(SplineParams.EndScale);
	Output.StartOffset 				= FVector2f(SplineParams.StartOffset);
	Output.EndOffset 				= FVector2f(SplineParams.EndOffset);
	Output.StartRoll 				= SplineParams.StartRoll;
	Output.EndRoll 					= SplineParams.EndRoll;
	Output.NaniteClusterBoundsScale	= SplineParams.NaniteClusterBoundsScale;
	Output.bSmoothInterpRollScale 	= bSmoothInterpRollScale;
	Output.SplineUpDir 				= FVector3f(SplineUpDir);
	Output.TextureCoord 			= FUintVector2(INDEX_NONE, INDEX_NONE); // either unused or assigned later

	const uint32 MeshXAxis = (ForwardAxis + 1) % 3;
	const uint32 MeshYAxis = (ForwardAxis + 2) % 3;
	Output.MeshDir = Output.MeshX = Output.MeshY = FVector3f::ZeroVector;
	Output.MeshDir[ForwardAxis] = 1.0f;
	Output.MeshX[MeshXAxis] = 1.0f;
	Output.MeshY[MeshYAxis] = 1.0f;

	Output.MeshZScale = 1.0f;
	Output.MeshZOffset = 0.0f;

	if (SourceMeshBounds.SphereRadius > 0.0f)
	{
		const float BoundsXYRadius = FVector3f(SourceMeshBounds.BoxExtent).Dot((Output.MeshX + Output.MeshY).GetUnsafeNormal());

		const float MeshMinZ = UE::SplineMesh::RealToFloatChecked(USplineMeshComponent::GetAxisValueRef(SourceMeshBounds.Origin - SourceMeshBounds.BoxExtent, ForwardAxis));
		const float MeshZLen = UE::SplineMesh::RealToFloatChecked(2 * USplineMeshComponent::GetAxisValueRef(SourceMeshBounds.BoxExtent, ForwardAxis));
		const float InvMeshZLen = (MeshZLen <= 0.0f) ? 1.0f : 1.0f / MeshZLen;
		constexpr float MeshTexelLen = float(SPLINE_MESH_TEXEL_WIDTH - 1);

		if (FMath::IsNearlyEqual(SplineBoundaryMin, SplineBoundaryMax))
		{
			Output.MeshZScale = InvMeshZLen;
			Output.MeshZOffset = -MeshMinZ * InvMeshZLen;
			Output.SplineDistToTexelScale = MeshTexelLen;
			Output.SplineDistToTexelOffset = 0.0f;
		}
		else
		{
			const float BoundaryLen = SplineBoundaryMax - SplineBoundaryMin;
			const float InvBoundaryLen = 1.0f / BoundaryLen;

			Output.MeshZScale = InvBoundaryLen;
			Output.MeshZOffset = -SplineBoundaryMin * InvBoundaryLen;
			Output.SplineDistToTexelScale = BoundaryLen * InvMeshZLen * MeshTexelLen;
			Output.SplineDistToTexelOffset = (SplineBoundaryMin - MeshMinZ) * InvMeshZLen * MeshTexelLen;
		}

		if (bNaniteEnabled || !CVarSkipUpdatingMeshDeformScaleForNonNaniteMeshes.GetValueOnAnyThread())
		{
			// Iteratively solve for an approximation of spline length
			float SplineLength = 0.0f;
			{
				static const uint32 NumIterations = 63; // 64 sampled points
				static const float IterStep = 1.0f / float(NumIterations);
				float A = 0.0f;
				FVector3f PrevPoint = SplineEvalPos(SplineParams, A);
				for (uint32 i = 0; i < NumIterations; ++i)
				{
					FVector3f Point = SplineEvalPos(SplineParams, A);
					SplineLength += (Point - PrevPoint).Length();
					PrevPoint = Point;
					A += IterStep;
				}
			}

			// Calculate an approximation of how much the mesh gets scaled in each local axis as a result of spline
			// deformation and take the smallest of the axes. This is important for LOD selection of Nanite spline
			// meshes.
			{
				// Estimate length added due to twisting as well
				const float XYRadius = BoundsXYRadius * FMath::Max(Output.StartScale.GetAbsMax(), Output.EndScale.GetAbsMax());
				const float TwistRadians = FMath::Abs(Output.StartRoll - Output.EndRoll);
				SplineLength += TwistRadians * XYRadius;

				// Take the mid-point scale in X/Y to balance out LOD selection in case either of them are extreme.
				auto AvgAbs = [](float A, float B) { return (FMath::Abs(A) + FMath::Abs(B)) * 0.5f; };
				const FVector3f DeformScale = FVector3f(
					SplineLength * Output.MeshZScale,
					AvgAbs(Output.StartScale.X, Output.EndScale.X),
					AvgAbs(Output.StartScale.Y, Output.EndScale.Y)
				);
			
				Output.MeshDeformScaleMinMax = FVector2f(DeformScale.GetMin(), DeformScale.GetMax());
			}
		}
		else
		{
			Output.MeshDeformScaleMinMax = FVector2f::One();
		}
	}
	return Output;
}

extern void InitSplineMeshVertexFactoryComponents(
	const FStaticMeshVertexBuffers& VertexBuffers, 
	const FSplineMeshVertexFactory* VertexFactory, 
	int32 LightMapCoordinateIndex, 
	bool bOverrideColorVertexBuffer,
	FLocalVertexFactory::FDataType& OutData);

static void InitSplineVertexFactory_Internal(FStaticMeshVertexFactories& VertexFactories, const FStaticMeshVertexBuffers& VertexBuffers, int32 LightMapCoordinateIndex, const ERHIFeatureLevel::Type FeatureLevel, FColorVertexBuffer* InOverrideColorVertexBuffer)
{
	// Skip LODs that have their render data stripped (eg. platform MinLod settings)
	if (VertexBuffers.StaticMeshVertexBuffer.GetNumVertices() == 0)
	{
		return;
	}

	bool bOverrideColorVertexBuffer = !!InOverrideColorVertexBuffer;

	if ((VertexFactories.SplineVertexFactory && !bOverrideColorVertexBuffer) || (VertexFactories.SplineVertexFactoryOverrideColorVertexBuffer && bOverrideColorVertexBuffer))
	{
		// we already have it
		return;
	}

	FSplineMeshVertexFactory* VertexFactory = new FSplineMeshVertexFactory(FeatureLevel);
	if (bOverrideColorVertexBuffer)
	{
		VertexFactories.SplineVertexFactoryOverrideColorVertexBuffer = VertexFactory;
	}
	else
	{
		VertexFactories.SplineVertexFactory = VertexFactory;
	}

	// Initialize the static mesh's vertex factory.
	ENQUEUE_RENDER_COMMAND(InitSplineMeshVertexFactory)(
		[VertexFactory, &VertexBuffers, bOverrideColorVertexBuffer, LightMapCoordinateIndex](FRHICommandListBase& RHICmdList)
		{
			FLocalVertexFactory::FDataType Data;
			InitSplineMeshVertexFactoryComponents(VertexBuffers, VertexFactory, LightMapCoordinateIndex, bOverrideColorVertexBuffer, Data);
			VertexFactory->SetData(RHICmdList, Data);
			VertexFactory->InitResource(RHICmdList);
		});
}

void FSplineMeshSceneProxyDesc::InitVertexFactory(UStaticMesh* Mesh, const ERHIFeatureLevel::Type FeatureLevel, int32 InLODIndex, FColorVertexBuffer* InOverrideColorVertexBuffer)
{	
	if (Mesh == nullptr)
	{
		return;
	}

	FStaticMeshRenderData* RenderData = Mesh->GetRenderData();

	InitSplineVertexFactory_Internal(RenderData->LODVertexFactories[InLODIndex], RenderData->LODResources[InLODIndex].VertexBuffers, Mesh->GetLightMapCoordinateIndex(), FeatureLevel, InOverrideColorVertexBuffer);
}

void FSplineMeshSceneProxyDesc::InitRayTracingProxyVertexFactory(UStaticMesh* Mesh, const ERHIFeatureLevel::Type FeatureLevel, int32 InLODIndex, FColorVertexBuffer* InOverrideColorVertexBuffer)
{
	if (Mesh == nullptr || Mesh->GetRenderData()->RayTracingProxy->bUsingRenderingLODs)
	{
		return;
	}

	FStaticMeshRenderData* RenderData = Mesh->GetRenderData();

	InitSplineVertexFactory_Internal((*RenderData->RayTracingProxy->LODVertexFactories)[InLODIndex], *RenderData->RayTracingProxy->LODs[InLODIndex].VertexBuffers, Mesh->GetLightMapCoordinateIndex(), FeatureLevel, InOverrideColorVertexBuffer);
}

static float ComputeRatioAlongSpline(const FBoxSphereBounds& SourceMeshBounds, ESplineMeshAxis::Type ForwardAxis, float SplineBoundaryMin, float SplineBoundaryMax, float DistanceAlong)
{
	// Find how far 'along' mesh (or custom boundaries) we are
	float Alpha = 0.f;

	const bool bHasCustomBoundary = !FMath::IsNearlyEqual(SplineBoundaryMin, SplineBoundaryMax);
	if (bHasCustomBoundary)
	{
		Alpha = (DistanceAlong - SplineBoundaryMin) / (SplineBoundaryMax - SplineBoundaryMin);
	}
	else if (SourceMeshBounds.SphereRadius > 0.0f)
	{
		const double MeshMinZ = USplineMeshComponent::GetAxisValueRef(SourceMeshBounds.Origin, ForwardAxis) - USplineMeshComponent::GetAxisValueRef(SourceMeshBounds.BoxExtent, ForwardAxis);
		const double MeshRangeZ = 2 * USplineMeshComponent::GetAxisValueRef(SourceMeshBounds.BoxExtent, ForwardAxis);
		if (MeshRangeZ > UE_SMALL_NUMBER)
		{
			Alpha = UE::SplineMesh::RealToFloatChecked((DistanceAlong - MeshMinZ) / MeshRangeZ);
		}
	}
	return Alpha;
}

FBox ComputeDistortedBounds(
	const FBoxSphereBounds& BoundsToDistort,
	const FBoxSphereBounds& SourceMeshBounds,
	const FSplineMeshParams& SplineParams,
	const FVector& SplineUpDir,
	ESplineMeshAxis::Type ForwardAxis,
	bool bSmoothInterpRollScale,
	bool bExtrapolate,
	float SplineBoundaryMin,
	float SplineBoundaryMax,
	float MinT,
	float MaxT,
	const FMatrix* LocalToMeshCards = nullptr,
	const FSplineMeshParams* MeshCardsSpaceSplineParams = nullptr)
{
	const FVector AxisMask = USplineMeshComponent::GetAxisMask(ForwardAxis);
	const FVector FlattenedBoundsOrigin = BoundsToDistort.Origin * AxisMask;
	const FVector FlattenedBoundsExtent = BoundsToDistort.BoxExtent * AxisMask;
	const FBox FlattenedBounds(FlattenedBoundsOrigin - FlattenedBoundsExtent, FlattenedBoundsOrigin + FlattenedBoundsExtent);

	auto GetDistortedSlice = [&](float Alpha)
	{
		FMatrix M = CalcSliceTransformAtSplineOffset(SplineParams, SplineUpDir, ForwardAxis, bSmoothInterpRollScale, Alpha, MinT, MaxT).ToMatrixWithScale();
		if (LocalToMeshCards)
		{
			M *= *LocalToMeshCards;
		}
		return FlattenedBounds.TransformBy(M);
	};

	FBox BoundingBox(ForceInit);
	BoundingBox += GetDistortedSlice(MinT);
	BoundingBox += GetDistortedSlice(MaxT);

	auto AppendAxisExtrema = [&](const double Discriminant, const double A, const double B)
	{
		// Negative discriminant means no solution; A == 0 implies coincident start/end points
		if (Discriminant > 0.0 && !FMath::IsNearlyZero(A))
		{
			const double SqrtDiscriminant = FMath::Sqrt(Discriminant);
			const double Denominator = 0.5 / A;
			const float T0 = UE::SplineMesh::RealToFloatChecked((-B + SqrtDiscriminant) * Denominator);
			const float T1 = (-B - SqrtDiscriminant) * Denominator;

			if (T0 > MinT && T0 < MaxT)
			{
				BoundingBox += GetDistortedSlice(T0);
			}

			if (T1 > MinT && T1 < MaxT)
			{
				BoundingBox += GetDistortedSlice(T1);
			}
		}
	};

	// Work out coefficients of the cubic spline derivative equation dx/dt
	const FSplineMeshParams& SolveParams = MeshCardsSpaceSplineParams ? *MeshCardsSpaceSplineParams : SplineParams;
	const FVector A = 6 * SolveParams.StartPos + 3 * SolveParams.StartTangent + 3 * SolveParams.EndTangent - 6 * SolveParams.EndPos;
	const FVector B = -6 * SolveParams.StartPos - 4 * SolveParams.StartTangent - 2 * SolveParams.EndTangent + 6 * SolveParams.EndPos;
	const FVector C = SolveParams.StartTangent;

	// Minima/maxima happen where dx/dt == 0, calculate t values
	const FVector Discriminant = B * B - 4 * A * C;

	// Work out minima/maxima component-by-component.
	AppendAxisExtrema(Discriminant.X, A.X, B.X);
	AppendAxisExtrema(Discriminant.Y, A.Y, B.Y);
	AppendAxisExtrema(Discriminant.Z, A.Z, B.Z);

	// Applying extrapolation if bounds to apply on spline are different than the mesh bounds used to define the spline range [0,1]
	if (bExtrapolate)
	{
		const double BoundsMin = USplineMeshComponent::GetAxisValueRef(BoundsToDistort.Origin - BoundsToDistort.BoxExtent, ForwardAxis);
		const double BoundsMax = USplineMeshComponent::GetAxisValueRef(BoundsToDistort.Origin + BoundsToDistort.BoxExtent, ForwardAxis);

		float Alpha = ComputeRatioAlongSpline(SourceMeshBounds, ForwardAxis, SplineBoundaryMin, SplineBoundaryMax, UE::SplineMesh::RealToFloatChecked(BoundsMin));
		if (Alpha < MinT)
		{
			BoundingBox += GetDistortedSlice(Alpha);
		}

		Alpha = ComputeRatioAlongSpline(SourceMeshBounds, ForwardAxis, SplineBoundaryMin, SplineBoundaryMax, UE::SplineMesh::RealToFloatChecked(BoundsMax));
		if (Alpha > MaxT)
		{
			BoundingBox += GetDistortedSlice(Alpha);
		}
	}

	return BoundingBox;
}

bool FSplineMeshSceneProxyDesc::ComputeDistortedBoundsForLumenCardCapture(const FVector& LocalToWorldScale, FBox& OutMeshCardsBounds, FTransform& OutMeshCardsToLocal) const
{
	const bool bUniformScale = LocalToWorldScale.IsUniform();
	FVector StartPos = SplineParams.StartPos;
	FVector EndPos = SplineParams.EndPos;
	FVector StartTangent = SplineParams.StartTangent;
	FVector EndTangent = SplineParams.EndTangent;
	FVector UpDirection = SplineUpDir;

	if (!bUniformScale)
	{
		// If local to world scale is non-uniform, the transform matrix from mesh cards space to world space
		// becomes non-orthogonal which Lumen card capture doesn't support. To workaround, we need to build
		// the new mesh cards space coordinate frame in the scaled local space.
		StartPos *= LocalToWorldScale;
		EndPos *= LocalToWorldScale;
		StartTangent *= LocalToWorldScale;
		EndTangent *= LocalToWorldScale;
		UpDirection *= LocalToWorldScale;
	}

	const FVector MeshCardsForward = (EndPos - StartPos).GetSafeNormal();
	const FVector MeshCardsRight = FVector::CrossProduct(UpDirection, MeshCardsForward).GetSafeNormal();
	const FVector MeshCardsUp = FVector::CrossProduct(MeshCardsForward, MeshCardsRight);
	const FVector& MeshCardsOrigin = StartPos;

	FMatrix MeshCardsToLocal;
	switch (ForwardAxis)
	{
	case ESplineMeshAxis::X:
		MeshCardsToLocal = FMatrix(MeshCardsForward, MeshCardsRight, MeshCardsUp, MeshCardsOrigin);
		break;
	case ESplineMeshAxis::Y:
		MeshCardsToLocal = FMatrix(MeshCardsUp, MeshCardsForward, MeshCardsRight, MeshCardsOrigin);
		break;
	case ESplineMeshAxis::Z:
		MeshCardsToLocal = FMatrix(MeshCardsRight, MeshCardsUp, MeshCardsForward, MeshCardsOrigin);
		break;
	default:
		checkNoEntry();
		MeshCardsToLocal = FMatrix::Identity;
		break;
	}

	OutMeshCardsToLocal.SetFromMatrix(MeshCardsToLocal);

	FSplineMeshParams MeshCardsSpaceSplineParams = SplineParams;
	MeshCardsSpaceSplineParams.StartPos = OutMeshCardsToLocal.InverseTransformPositionNoScale(StartPos);
	MeshCardsSpaceSplineParams.EndPos = OutMeshCardsToLocal.InverseTransformPositionNoScale(EndPos);
	MeshCardsSpaceSplineParams.StartTangent = OutMeshCardsToLocal.InverseTransformVectorNoScale(StartTangent);
	MeshCardsSpaceSplineParams.EndTangent = OutMeshCardsToLocal.InverseTransformVectorNoScale(EndTangent);

	float MinT = 0.0f;
	float MaxT = 1.0f;
	ComputeVisualMeshSplineTRange(MinT, MaxT);

	if (bUniformScale)
	{
		// Evaluate and distort bounds directly in mesh cards space
		const FVector MeshCardsSpaceSplineUp = OutMeshCardsToLocal.InverseTransformVectorNoScale(UpDirection);

		OutMeshCardsBounds = ::ComputeDistortedBounds(
			SourceMeshBounds,
			SourceMeshBounds,
			MeshCardsSpaceSplineParams,
			MeshCardsSpaceSplineUp,
			ForwardAxis,
			bSmoothInterpRollScale,
			false /*bExtrapolate*/,
			SplineBoundaryMin,
			SplineBoundaryMax,
			MinT,
			MaxT);
	}
	else
	{
		// When the scale isn't uniform, we cannot calculate slice transform in mesh cards space because some
		// some operations such as box transform, roll, and offset will be incorrect. Instead, we evaluate
		// the spline and distort the bounds in local space. Then transform the local space box into mesh cards
		// space and use it to expand the bbox. 
		const FMatrix InverseRotation(
			MeshCardsToLocal.GetColumn(0),
			MeshCardsToLocal.GetColumn(1),
			MeshCardsToLocal.GetColumn(2),
			FVector::ZeroVector);

		const FMatrix LocalToMeshCards(
			LocalToWorldScale.X * InverseRotation.GetScaledAxis(EAxis::X),
			LocalToWorldScale.Y * InverseRotation.GetScaledAxis(EAxis::Y),
			LocalToWorldScale.Z * InverseRotation.GetScaledAxis(EAxis::Z),
			FVector(InverseRotation.TransformVector(-MeshCardsToLocal.GetOrigin())));

		OutMeshCardsBounds = ::ComputeDistortedBounds(
			SourceMeshBounds,
			SourceMeshBounds,
			SplineParams,
			SplineUpDir,
			ForwardAxis,
			bSmoothInterpRollScale,
			false /*bExtrapolate*/,
			SplineBoundaryMin,
			SplineBoundaryMax,
			MinT,
			MaxT,
			&LocalToMeshCards,
			&MeshCardsSpaceSplineParams);
	}

	return !bUniformScale;
}

FBox FSplineMeshSceneProxyDesc::ComputeDistortedBounds(const FTransform& InLocalToWorld, const FBoxSphereBounds& InMeshBounds, const FBoxSphereBounds* InBoundsToDistort) const
{
	float MinT = 0.0f;
	float MaxT = 1.0f;
	ComputeVisualMeshSplineTRange(MinT, MaxT);

	const FBoxSphereBounds& BoundsToDistort = InBoundsToDistort ? *InBoundsToDistort : InMeshBounds;
	const bool bExtrapolate = InBoundsToDistort != nullptr && InBoundsToDistort != &InMeshBounds;

	const FBox BoundingBox = ::ComputeDistortedBounds(
		BoundsToDistort,
		SourceMeshBounds,
		SplineParams,
		SplineUpDir,
		ForwardAxis,
		bSmoothInterpRollScale,
		bExtrapolate,
		SplineBoundaryMin,
		SplineBoundaryMax,
		MinT,
		MaxT);

	return BoundingBox.TransformBy(InLocalToWorld);
}

FTransform FSplineMeshSceneProxyDesc::CalcSliceTransform(const float DistanceAlong) const
{
	const float Alpha = ComputeRatioAlongSpline(DistanceAlong);
	

	float MinT = 0.f;
	float MaxT = 1.f;
	ComputeVisualMeshSplineTRange(MinT, MaxT);

	return CalcSliceTransformAtSplineOffset(Alpha, MinT, MaxT);
}

FTransform FSplineMeshSceneProxyDesc::CalcSliceTransformAtSplineOffset(const float Alpha, const float MinT, const float MaxT) const
{
	return ::CalcSliceTransformAtSplineOffset(SplineParams, SplineUpDir, ForwardAxis, bSmoothInterpRollScale, Alpha, MinT, MaxT);
}

float FSplineMeshSceneProxyDesc::ComputeRatioAlongSpline(float DistanceAlong) const
{
	return ::ComputeRatioAlongSpline(SourceMeshBounds, ForwardAxis, SplineBoundaryMin, SplineBoundaryMax, DistanceAlong);
}

void FSplineMeshSceneProxyDesc::ComputeVisualMeshSplineTRange(float& MinT, float& MaxT) const
{
	MinT = 0.0;
	MaxT = 1.0;	
	const bool bHasCustomBoundary = !FMath::IsNearlyEqual(SplineBoundaryMin, SplineBoundaryMax);
	if (bHasCustomBoundary)
	{
		// If there's a custom boundary, alter the min/max of the spline we need to evaluate
		ESplineMeshAxis::Type ForwardAxisType = static_cast<ESplineMeshAxis::Type>(ForwardAxis);
		const float BoundsMin = UE::SplineMesh::RealToFloatChecked(USplineMeshComponent::GetAxisValueRef(SourceMeshBounds.Origin - SourceMeshBounds.BoxExtent, ForwardAxisType));
		const float BoundsMax = UE::SplineMesh::RealToFloatChecked(USplineMeshComponent::GetAxisValueRef(SourceMeshBounds.Origin + SourceMeshBounds.BoxExtent, ForwardAxisType));
		const float BoundsMinT = (BoundsMin - SplineBoundaryMin) / (SplineBoundaryMax - SplineBoundaryMin);
		const float BoundsMaxT = (BoundsMax - SplineBoundaryMin) / (SplineBoundaryMax - SplineBoundaryMin);

		// Disallow extrapolation beyond a certain value; enormous bounding boxes cause the render thread to crash
		constexpr float MaxSplineExtrapolation = 4.0f;
		MinT = FMath::Max(-MaxSplineExtrapolation, BoundsMinT);
		MaxT = FMath::Min(BoundsMaxT, MaxSplineExtrapolation);
	}
}