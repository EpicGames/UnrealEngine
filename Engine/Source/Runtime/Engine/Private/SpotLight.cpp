// Copyright Epic Games, Inc. All Rights Reserved.

#include "Engine/SpotLight.h"
#include "Async/TaskGraphInterfaces.h"
#include "Components/ArrowComponent.h"
#include "Components/SpotLightComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SpotLight)

ASpotLight::ASpotLight(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer.SetDefaultSubobjectClass<USpotLightComponent>(TEXT("LightComponent0")))
{
	// Structure to hold one-time initialization
	struct FConstructorStatics
	{
		FName ID_Lighting;
		FText NAME_Lighting;
		FConstructorStatics()
			: ID_Lighting(TEXT("Lighting"))
			, NAME_Lighting(NSLOCTEXT( "SpriteCategory", "Lighting", "Lighting" ))
		{
		}
	};
	static FConstructorStatics ConstructorStatics;

	SpotLightComponent = CastChecked<USpotLightComponent>(GetLightComponent());
	SpotLightComponent->Mobility = EComponentMobility::Stationary;
	SpotLightComponent->SetRelativeRotation(FRotator(-90, 0, 0));

	RootComponent = SpotLightComponent;

#if WITH_EDITORONLY_DATA
	ArrowComponent = CreateEditorOnlyDefaultSubobject<UArrowComponent>(TEXT("ArrowComponent0"));
	if (ArrowComponent)
	{
		ArrowComponent->ArrowColor = GetLightColor().ToFColor(true);
		ArrowComponent->bTreatAsASprite = true;
		ArrowComponent->SpriteInfo.Category = ConstructorStatics.ID_Lighting;
		ArrowComponent->SpriteInfo.DisplayName = ConstructorStatics.NAME_Lighting;
		ArrowComponent->SetupAttachment(SpotLightComponent);
		ArrowComponent->bLightAttachment = true;
		ArrowComponent->bIsScreenSizeScaled = true;
	}
#endif // WITH_EDITORONLY_DATA
}

void ASpotLight::PostLoad()
{
	Super::PostLoad();

#if WITH_EDITORONLY_DATA
	if(ArrowComponent)
	{
		ArrowComponent->ArrowColor = GetLightColor().ToFColor(true);
	}
#endif
}

#if WITH_EDITOR
void ASpotLight::LoadedFromAnotherClass(const FName& OldClassName)
{
	Super::LoadedFromAnotherClass(OldClassName);

	if(GetLinkerUEVersion() < VER_UE4_REMOVE_LIGHT_MOBILITY_CLASSES)
	{
		static FName SpotLightStatic_NAME("SpotLightStatic");
		static FName SpotLightMovable_NAME("SpotLightMovable");
		static FName SpotLightStationary_NAME("SpotLightStationary");

		check(GetLightComponent() != NULL);

		if(OldClassName == SpotLightStatic_NAME)
		{
			GetLightComponent()->Mobility = EComponentMobility::Static;
		}
		else if(OldClassName == SpotLightMovable_NAME)
		{
			GetLightComponent()->Mobility = EComponentMobility::Movable;
		}
		else if(OldClassName == SpotLightStationary_NAME)
		{
			GetLightComponent()->Mobility = EComponentMobility::Stationary;
		}
	}
}
#endif // WITH_EDITOR

void ASpotLight::SetInnerConeAngle(float NewInnerConeAngle)
{
	SpotLightComponent->SetInnerConeAngle(NewInnerConeAngle);
}

void ASpotLight::SetOuterConeAngle(float NewOuterConeAngle)
{
	SpotLightComponent->SetOuterConeAngle(NewOuterConeAngle);
}

// Disable for now
//void ASpotLight::SetLightShaftConeAngle(float NewLightShaftConeAngle)
//{
//	SpotLightComponent->SetLightShaftConeAngle(NewLightShaftConeAngle);
//}

#if WITH_EDITOR
void ASpotLight::EditorApplyScale(const FVector& DeltaScale, const FVector* PivotLocation, bool bAltDown, bool bShiftDown, bool bCtrlDown)
{
	const FVector ModifiedScale = DeltaScale * ( AActor::bUsePercentageBasedScaling ? 10000.0f : 100.0f );

	if ( bCtrlDown )
	{
		FMath::ApplyScaleToFloat( SpotLightComponent->OuterConeAngle, ModifiedScale, 0.01f );
		SpotLightComponent->OuterConeAngle = FMath::Min( 89.0f, SpotLightComponent->OuterConeAngle );
		SpotLightComponent->InnerConeAngle = FMath::Min( SpotLightComponent->OuterConeAngle, SpotLightComponent->InnerConeAngle );
	}
	else if ( bAltDown )
	{
		FMath::ApplyScaleToFloat( SpotLightComponent->InnerConeAngle, ModifiedScale, 0.01f );
		SpotLightComponent->InnerConeAngle = FMath::Min( 89.0f, SpotLightComponent->InnerConeAngle );
		SpotLightComponent->OuterConeAngle = FMath::Max( SpotLightComponent->OuterConeAngle, SpotLightComponent->InnerConeAngle );
	}
	else
	{
		FMath::ApplyScaleToFloat( SpotLightComponent->AttenuationRadius, ModifiedScale );
	}

	PostEditChange();
}

void ASpotLight::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	if(ArrowComponent)
	{
		ArrowComponent->ArrowColor = GetLightColor().ToFColor(true);
	}
}
#endif


