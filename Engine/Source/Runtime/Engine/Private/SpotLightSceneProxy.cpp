// Copyright Epic Games, Inc. All Rights Reserved.

#include "SpotLightSceneProxy.h"
#include "SpotLightSceneProxyDesc.h"
#include "Components/SpotLightComponent.h"
#include "SceneInterface.h"
#include "SceneManagement.h"
#include "SceneView.h"

FSpotLightSceneProxy::FSpotLightSceneProxy(const USpotLightComponent* Component)
	: FSpotLightSceneProxy(USpotLightComponent::BuildSceneProxyDesc(*Component))
{
}

FSpotLightSceneProxy::FSpotLightSceneProxy(const FSpotLightSceneProxyDesc& LightDesc)
	: FPointLightSceneProxy(LightDesc)
{
	const FVector2f ClampedConeAngles = USpotLightComponent::GetClampedConeAngles(LightDesc.InnerConeAngle, LightDesc.OuterConeAngle);
	OuterConeAngle = ClampedConeAngles.Y;
	CosOuterCone = FMath::Cos(ClampedConeAngles.Y);
	SinOuterCone = FMath::Sin(ClampedConeAngles.Y);
	CosInnerCone = FMath::Cos(ClampedConeAngles.X);
	InvCosConeDifference = 1.0f / (CosInnerCone - CosOuterCone);
	InvTanOuterCone = 1.0f / FMath::Tan(ClampedConeAngles.Y);
}

void FSpotLightSceneProxy::GetLightShaderParameters(FLightRenderParameters& LightParameters, uint32 Flags) const
{
	LightParameters.WorldPosition = GetOrigin();
	LightParameters.InvRadius = InvRadius;
	LightParameters.Color = GetColor();
	LightParameters.FalloffExponent = bInverseSquared ? 0.0f : FalloffExponent;
	LightParameters.Direction = FVector3f(-GetDirection());
	LightParameters.Tangent = FVector3f(WorldToLight.M[0][2], WorldToLight.M[1][2], WorldToLight.M[2][2]);
	LightParameters.SpotAngles = FVector2f(CosOuterCone, InvCosConeDifference);
	LightParameters.SpecularScale = FMath::Clamp(SpecularScale, 0.f, 1.f);
	LightParameters.DiffuseScale = FMath::Clamp(DiffuseScale, 0.f, 1.f);
	LightParameters.SourceRadius = SourceRadius;
	LightParameters.SoftSourceRadius = SoftSourceRadius;
	LightParameters.SourceLength = SourceLength;
	LightParameters.RectLightBarnCosAngle = 0.0f;
	LightParameters.RectLightBarnLength = -2.0f;
	LightParameters.RectLightAtlasUVOffset = FVector2f::ZeroVector;
	LightParameters.RectLightAtlasUVScale = FVector2f::ZeroVector;
	LightParameters.RectLightAtlasMaxLevel = FLightRenderParameters::GetRectLightAtlasInvalidMIPLevel();
	LightParameters.IESAtlasIndex = INDEX_NONE;
	LightParameters.LightFunctionAtlasLightIndex = GetLightFunctionAtlasLightIndex();
	LightParameters.bAffectsTranslucentLighting = AffectsTranslucentLighting() ? 1 : 0;
	LightParameters.InverseExposureBlend = InverseExposureBlend;

	if (IESAtlasId != uint32(INDEX_NONE))
	{
		GetSceneInterface()->GetLightIESAtlasSlot(this, &LightParameters);
	}
}

bool FSpotLightSceneProxy::AffectsBounds(const FBoxSphereBounds& Bounds) const
{
	if (!FLocalLightSceneProxy::AffectsBounds(Bounds))
	{
		return false;
	}

	FVector	U = GetOrigin() - (Bounds.SphereRadius / SinOuterCone) * GetDirection(),
		D = Bounds.Origin - U;
	float	dsqr = D | D,
		E = GetDirection() | D;
	if (E > 0.0f && E * E >= dsqr * FMath::Square(CosOuterCone))
	{
		D = Bounds.Origin - GetOrigin();
		dsqr = D | D;
		E = -(GetDirection() | D);
		if (E > 0.0f && E * E >= dsqr * FMath::Square(SinOuterCone))
			return dsqr <= FMath::Square(Bounds.SphereRadius);
		else
			return true;
	}

	return false;
}

/**
 * Sets up a projected shadow initializer for shadows from the entire scene.
 * @return True if the whole-scene projected shadow should be used.
 */
bool FSpotLightSceneProxy::GetWholeSceneProjectedShadowInitializer(const FSceneViewFamily& ViewFamily, TArray<FWholeSceneProjectedShadowInitializer, TInlineAllocator<6> >& OutInitializers) const
{
	FWholeSceneProjectedShadowInitializer& OutInitializer = OutInitializers.AddDefaulted_GetRef();
	OutInitializer.PreShadowTranslation = -GetLightToWorld().GetOrigin();
	OutInitializer.WorldToLight = GetWorldToLight().RemoveTranslation();
	OutInitializer.Scales = FVector2D(InvTanOuterCone, InvTanOuterCone);

	const FSphere AbsoluteBoundingSphere = FSpotLightSceneProxy::GetBoundingSphere();
	OutInitializer.SubjectBounds = FBoxSphereBounds(
		AbsoluteBoundingSphere.Center - GetOrigin(),
		FVector(AbsoluteBoundingSphere.W, AbsoluteBoundingSphere.W, AbsoluteBoundingSphere.W),
		AbsoluteBoundingSphere.W
	);

	OutInitializer.WAxis = FVector4(0, 0, 1, 0);
	OutInitializer.MinLightW = 0.1f;
	OutInitializer.MaxDistanceToCastInLightW = Radius;
	OutInitializer.bRayTracedDistanceField = UseRayTracedDistanceFieldShadows() && DoesPlatformSupportDistanceFieldShadowing(ViewFamily.GetShaderPlatform());
	return true;
}

float FSpotLightSceneProxy::GetOuterConeAngle() const 
{
	return OuterConeAngle;
}

FSphere FSpotLightSceneProxy::GetBoundingSphere() const
{
	return FMath::ComputeBoundingSphereForCone(GetOrigin(), GetDirection(), (FSphere::FReal)Radius, (FSphere::FReal)CosOuterCone, (FSphere::FReal)SinOuterCone);
}

float FSpotLightSceneProxy::GetEffectiveScreenRadius(const FViewMatrices& ShadowViewMatrices, const FIntPoint& CameraViewRectSize) const
{
	// Heuristic: use the radius of the inscribed sphere at the cone's end as the light's effective screen radius
	// We do so because we do not want to use the light's radius directly, which will make us overestimate the shadow map resolution greatly for a spot light

	// In the correct form,
	//   InscribedSpherePosition = GetOrigin() + GetDirection() * GetRadius() / CosOuterCone
	//   InscribedSphereRadius = GetRadius() / SinOuterCone
	// Do it incorrectly to avoid division which is more expensive and risks division by zero
	const FVector InscribedSpherePosition = GetOrigin() + GetDirection() * GetRadius() * CosOuterCone;
	const float InscribedSphereRadius = GetRadius() * SinOuterCone;

	const float SphereDistanceFromViewOrigin = (InscribedSpherePosition - ShadowViewMatrices.GetViewOrigin()).Size();

	const FVector2D& ProjectionScale = ShadowViewMatrices.GetProjectionScale();
	const float ScreenScale = FMath::Max(CameraViewRectSize.X * 0.5f * ProjectionScale.X, CameraViewRectSize.Y * 0.5f * ProjectionScale.Y);

	return ScreenScale * InscribedSphereRadius / FMath::Max(SphereDistanceFromViewOrigin, 1.0f);
}