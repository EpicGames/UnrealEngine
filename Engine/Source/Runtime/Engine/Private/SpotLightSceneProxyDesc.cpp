// Copyright Epic Games, Inc. All Rights Reserved.

#include "SpotLightSceneProxyDesc.h"

void FSpotLightSceneProxyDesc::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

	Ar << InnerConeAngle;
	Ar << OuterConeAngle;
}