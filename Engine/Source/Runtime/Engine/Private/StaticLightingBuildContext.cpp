// Copyright Epic Games, Inc. All Rights Reserved.

#include "StaticLightingBuildContext.h"

#include "Engine/Level.h"
#include "Engine/World.h"
#include "GameFramework/Actor.h"
#include "WorldPartition/StaticLightingData/VolumetricLightmapGrid.h"
#include "Engine/MapBuildDataRegistry.h"
#include "WorldPartition/WorldPartition.h"
#include "PrecomputedVolumetricLightmap.h"
#include "LevelInstance/LevelInstanceSubsystem.h"
#include "WorldPartition/StaticLightingData/StaticLightingDescriptors.h"

#if WITH_EDITOR

FStaticLightingBuildContext::FStaticLightingBuildContext(UWorld* InWorld, ULevel* InLightingScenario) 	
{
	checkf(InWorld, TEXT("Constructing a FStaticLightingBuildContext requires a valid World"));

	World = InWorld;
	LightingScenario = InLightingScenario;

	MapBuildDataRegistry = nullptr;
	ULevel* GlobalRegistryLevel = (InLightingScenario) ? InLightingScenario : InWorld->PersistentLevel.Get();
	
	// If the Level already has MapBuildData only reuse it if it still has it's standlone flag 
	// this ensures calls to GetOrCreateMapBuildData() on the Level will not create a new MapBuildData
	if (GlobalRegistryLevel->MapBuildData && GlobalRegistryLevel->MapBuildData->HasAllFlags(RF_Standalone|RF_Public))
	{
		MapBuildDataRegistry = GlobalRegistryLevel->MapBuildData;
	}

	for (int32 LevelIndex = 0; LevelIndex < World->GetNumLevels(); LevelIndex++)
	{
		ULevel* Level = World->GetLevel(LevelIndex);
		FGuid LevelGuid = FGuid(0, 0, 0, LevelIndex);
		LevelGuids.Add(LevelGuid, Level);
	}
	
	FGuid FirstGuid = FGuid(0, 0, 0, 0);
	check(GetLevelForGuid(FirstGuid) == World->PersistentLevel);

	if (InWorld->IsPartitionedWorld())
	{
		VolumetricLightMapGridDesc = new FVolumetricLightMapGridDesc;
		VolumetricLightMapGridDesc->Initialize(InWorld, InWorld->GetWorldPartition()->GetRuntimeWorldBounds());
		Descriptors = FStaticLightingDescriptors::Get();
	}	
	else
	{
		VolumetricLightMapGridDesc = nullptr;
		Descriptors = nullptr;
	}
}

FStaticLightingBuildContext::FStaticLightingBuildContext(FStaticLightingBuildContext&& InFrom)
{
	MapBuildDataRegistry = InFrom.MapBuildDataRegistry;
	LevelGuids = MoveTemp(InFrom.LevelGuids);
	World = InFrom.World;
	LightingScenario = InFrom.LightingScenario;
	ImportanceBounds = InFrom.ImportanceBounds;
	LocalToGlobalIndirectionOffset = InFrom.LocalToGlobalIndirectionOffset;
	VolumetricLightMapGridDesc = InFrom.VolumetricLightMapGridDesc;
	Descriptors = InFrom.Descriptors;

	InFrom.VolumetricLightMapGridDesc = nullptr;
}

FStaticLightingBuildContext::~FStaticLightingBuildContext()
{
	delete VolumetricLightMapGridDesc;	
}

void FStaticLightingBuildContext::SetImportanceBounds(const FBox& Bounds)
{
	// ImportanceBounds passed to Lightmass, may not encompass the whole world (for 
	// example when using distributed VLM computations). 
	// All indirections are local this value, so we recompute an indirection offset
	// to be able to move our local results to world results	
	check(VolumetricLightMapGridDesc);
	ImportanceBounds = Bounds;

	FVector Offset =  ImportanceBounds.Min - VolumetricLightMapGridDesc->GridBounds.Min;
	LocalToGlobalIndirectionOffset = (FIntVector)(Offset / VolumetricLightMapGridDesc->BrickSize);
}

bool FStaticLightingBuildContext::ShouldIncludeActor(AActor* Actor) const
{
	check(Actor);

	ULevel* ActorLevel = Actor->GetLevel();
	check(ActorLevel);

	bool IncludeActor = false;

	if (!LightingScenario || !ActorLevel->bIsLightingScenario || ActorLevel == LightingScenario)
	{
		IncludeActor = true;
	}

	return IncludeActor;
}

bool FStaticLightingBuildContext::ShouldIncludeLevel(ULevel* Level) const
{
	check(Level);

	bool IncludeLevel = false;

	if (!LightingScenario || !Level->bIsLightingScenario || Level == LightingScenario)
	{
		IncludeLevel = true;
	}

	return IncludeLevel;
}

UMapBuildDataRegistry* FStaticLightingBuildContext::GetOrCreateGlobalRegistry() const
{
	if (!MapBuildDataRegistry)
	{
		if (LightingScenario)
		{
			MapBuildDataRegistry = LightingScenario->GetOrCreateMapBuildData();
		}
		else
		{
			MapBuildDataRegistry = World->PersistentLevel->GetOrCreateMapBuildData();
		}
	}
	
	return MapBuildDataRegistry;
}

ULevel* FStaticLightingBuildContext::GetLightingStorageLevel(ULevel* Level) const
{
	if (LightingScenario)
	{
		return LightingScenario;
	}
	else
	{
		return Level;
	}
}

FGuid FStaticLightingBuildContext::GetPersistentLevelGuid() const
{
	return *LevelGuids.FindKey(World->PersistentLevel);
}

FGuid FStaticLightingBuildContext::GetLevelGuidForLevel(ULevel* Level) const
{
	return *LevelGuids.FindKey(Level);
}

FPrecomputedVolumetricLightmapData& FStaticLightingBuildContext::GetOrCreateLevelPrecomputedVolumetricLightmapBuildData(const FGuid& LevelId) const
{	
	if (VolumetricLightMapGridDesc && LevelId.IsValid())
	{
		if (FVolumetricLightMapGridCell* Cell = VolumetricLightMapGridDesc->GetCell(LevelId))
		{	
			if (!Cell->EditorData)
			{
				Cell->EditorData = new FPrecomputedVolumetricLightmapData();
			}
			return *Cell->EditorData;
		}
	}

	UMapBuildDataRegistry* Registry = nullptr;
	if (LightingScenario)
	{
		Registry = LightingScenario->GetOrCreateMapBuildData();
	}
	else
	{
		Registry = GetOrCreateRegistryForLevelGuid(LevelId);
	}
	
	FGuid BuildDataID = GetLevelBuildDataID(LevelId);
	if (FPrecomputedVolumetricLightmapData* Data = Registry->GetLevelPrecomputedVolumetricLightmapBuildData(BuildDataID))
	{
		return *Data;
	}
	
	return Registry->AllocateLevelPrecomputedVolumetricLightmapBuildData(BuildDataID);
}

FGuid FStaticLightingBuildContext::GetLevelGuidForVLMBrick(const FIntVector& BrickCoordinates) const
{
	FVolumetricLightMapGridDesc& Desc = *VolumetricLightMapGridDesc;
	FVector BrickInWorld = (FVector)(BrickCoordinates * Desc.BrickSize) + Desc.GridBounds.Min;
	BrickInWorld += FVector(Desc.DetailCellSize/2, Desc.DetailCellSize/2, Desc.DetailCellSize/2);	// offset by half detail cell size to avoid being on the edge
	
	if (FVolumetricLightMapGridCell* Cell = Desc.GetCell(BrickInWorld))
	{
		return Desc.GetCellGuid(Cell->CellID);
	}
	
	return FGuid();
}


FGuid FStaticLightingBuildContext::GetLevelGuidForActor(AActor* Actor) const
{		
	check(Actor);
		
	if (!World->IsPartitionedWorld())
	{
		return *LevelGuids.FindKey(Actor->GetLevel());
	}

	FGuid LevelGuid = FGuid(0, 0, 0, 0);		
	return LevelGuid;
}


UMapBuildDataRegistry* FStaticLightingBuildContext::GetOrCreateRegistryForLevelGuid(const FGuid& Guid) const
{	
	if (Guid.IsValid())
	{	
		if (!World->IsPartitionedWorld())
		{
			ULevel* Level = GetLevelForGuid(Guid).Get();
			return Level->GetOrCreateMapBuildData();
		}
		else
		{
		}
	}

	UMapBuildDataRegistry* Registry = GetOrCreateGlobalRegistry();
	return Registry;
}

const TWeakObjectPtr<ULevel> FStaticLightingBuildContext::GetLevelForGuid(const FGuid& Guid) const
{	
	return LevelGuids.FindRef(Guid);
}

FGuid FStaticLightingBuildContext::GetLevelBuildDataID(const FGuid& LevelGuid) const
{
	if (!World->IsPartitionedWorld())
	{
		return GetLevelForGuid(LevelGuid)->LevelBuildDataId;
	}
	else
	{
		if (!LevelGuid.IsValid())
		{
			return World->PersistentLevel->LevelBuildDataId;
		}
		
		return LevelGuid;			
	}
}

UMapBuildDataRegistry* FStaticLightingBuildContext::GetOrCreateRegistryForActor(AActor* Actor) const
{
	check(Actor);
	UMapBuildDataRegistry* Registry = nullptr;

	if (Descriptors)
	{
		Registry = Descriptors->GetOrCreateRegistryForActor(Actor);
	}

	if (!Registry)
	{
		// For Actors in LevelInstances we need to defer storage to the level that owns the LevelInstance
		ULevel* Level = ULevelInstanceSubsystem::GetOwningLevel(Actor->GetLevel(), true);

		Registry = GetLightingStorageLevel(Level)->GetOrCreateMapBuildData();
	}
	
	UE_LOG_MAPBUILDDATA(Log, TEXT("Creating/Returning Registry %s for Actor %s, %s"), *Registry->GetFullName(), *Actor->GetActorNameOrLabel(), *Actor->GetFullName());

	return Registry;
}

UMapBuildDataRegistry* FStaticLightingBuildContext::GetRegistryForLevel(ULevel* Level) const
{
	// For  we need to defer storage to the level that owns the LevelInstance
	return GetLightingStorageLevel(ULevelInstanceSubsystem::GetOwningLevel(Level, true))->MapBuildData;
}

UMapBuildDataRegistry* FStaticLightingBuildContext::GetOrCreateRegistryForLevel(ULevel* Level) const
{
	// For  we need to defer storage to the level that owns the LevelInstance
	return GetLightingStorageLevel(ULevelInstanceSubsystem::GetOwningLevel(Level, true))->GetOrCreateMapBuildData();
}

#endif
