// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	StaticMesh.cpp: Static mesh class implementation.
=============================================================================*/

#include "Engine/StaticMesh.h"
#include "BodySetupEnums.h"
#include "Serialization/LargeMemoryWriter.h"
#include "Engine/StaticMeshSourceData.h"
#include "Misc/ConfigCacheIni.h"
#include "EngineLogs.h"
#include "Misc/ScopedSlowTask.h"
#include "Engine/CollisionProfile.h"
#include "Math/ScaleRotationTranslationMatrix.h"
#include "Serialization/MemoryReader.h"
#include "UObject/AssetRegistryTagsContext.h"
#include "UObject/EditorObjectVersion.h"
#include "PhysicsEngine/BoxElem.h"
#include "UObject/FrameworkObjectVersion.h"
#include "RenderUtils.h"
#include "UObject/Package.h"
#include "SceneInterface.h"
#include "UObject/DevObjectVersion.h"
#include "UObject/DebugSerializationFlags.h"
#include "UObject/UE5ReleaseStreamObjectVersion.h"
#include "UObject/FortniteMainBranchObjectVersion.h"
#include "UObject/UObjectAnnotation.h"
#include "UObject/ICookInfo.h"
#include "EngineUtils.h"
#include "Engine/AssetUserData.h"
#include "StaticMeshDescription.h"
#include "StaticMeshOperations.h"
#include "Rendering/NaniteResources.h"
#include "Rendering/NaniteCoarseMeshStreamingManager.h"
#include "Rendering/RayTracingGeometryManager.h"
#include "NaniteVertexFactory.h"
#include "SpeedTreeWind.h"
#include "DistanceFieldAtlas.h"
#include "MeshCardBuild.h"
#include "MeshCardRepresentation.h"
#include "PhysicsEngine/PhysicsSettings.h"
#include "PhysicsEngine/BodySetup.h"
#include "Engine/Engine.h"
#include "DynamicMeshBuilder.h"
#include "Model.h"
#include "Async/Async.h"
#include "SplineMeshSceneProxy.h"
#include "PSOPrecache.h"
#include "UObject/UObjectIterator.h"

#if WITH_EDITOR
#include "Framework/Notifications/NotificationManager.h"
#include "RawMesh.h"
#include "MeshBudgetProjectSettings.h"
#include "NaniteBuilder.h"
#include "MeshUtilitiesCommon.h"
#include "DerivedDataCacheInterface.h"
#include "PlatformInfo.h"
#include "ScopedTransaction.h"
#include "IMeshBuilderModule.h"
#include "IMeshReductionManagerModule.h"
#include "IMeshReductionInterfaces.h"
#include "StaticMeshCompiler.h"
#include "ObjectCacheContext.h"
#include "Misc/DataValidation.h"
#include "Engine/Texture2D.h"
#include "Hash/xxhash.h"

#include "DerivedDataCache.h"
#include "DerivedDataRequestOwner.h"
#include "DataDrivenShaderPlatformInfo.h"

#include "Framework/Docking/TabManager.h"
#include "Widgets/Notifications/SNotificationList.h"
#else
#include "Interfaces/ITargetPlatform.h"
#endif // #if WITH_EDITOR

#include "MaterialCachedData.h"
#include "Engine/StaticMeshSocket.h"
#include "MaterialDomain.h"
#include "EditorFramework/AssetImportData.h"
#include "AI/Navigation/NavCollisionBase.h"
#include "AI/NavigationSystemBase.h"
#include "ProfilingDebugging/CookStats.h"
#include "Streaming/UVChannelDensity.h"
#include "Logging/MessageLog.h"
#include "Logging/StructuredLog.h"
#include "Misc/UObjectToken.h"
#include "Streaming/StaticMeshUpdate.h"
#include "Streaming/TextureStreamingHelpers.h"
#include "ProfilingDebugging/IoStoreTrace.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(StaticMesh)

#if WITH_EDITORONLY_DATA
#include "Materials/Material.h"
#endif

#define LOCTEXT_NAMESPACE "StaticMesh"
DEFINE_LOG_CATEGORY(LogStaticMesh);	

DECLARE_MEMORY_STAT( TEXT( "StaticMesh Total Memory" ), STAT_StaticMeshTotalMemory2, STATGROUP_MemoryStaticMesh );
DECLARE_MEMORY_STAT( TEXT( "StaticMesh Vertex Memory" ), STAT_StaticMeshVertexMemory, STATGROUP_MemoryStaticMesh );
DECLARE_MEMORY_STAT( TEXT( "StaticMesh VxColor Resource Mem" ), STAT_ResourceVertexColorMemory, STATGROUP_MemoryStaticMesh );
DECLARE_MEMORY_STAT( TEXT( "StaticMesh Index Memory" ), STAT_StaticMeshIndexMemory, STATGROUP_MemoryStaticMesh );

DECLARE_MEMORY_STAT( TEXT( "StaticMesh Total Memory" ), STAT_StaticMeshTotalMemory, STATGROUP_Memory );


DECLARE_FLOAT_ACCUMULATOR_STAT(TEXT("UStaticMesh::Serialize"), STAT_StaticMesh_SerializeFull, STATGROUP_LoadTime);
DECLARE_FLOAT_ACCUMULATOR_STAT(TEXT("UStaticMesh_SerializeParent"), STAT_StaticMesh_SerializeParent, STATGROUP_LoadTime);
DECLARE_FLOAT_ACCUMULATOR_STAT(TEXT("UStaticMesh_RenderDataLoad"), STAT_StaticMesh_RenderData, STATGROUP_LoadTime);
DECLARE_FLOAT_ACCUMULATOR_STAT(TEXT("UStaticMesh_RenderDataFixup"), STAT_StaticMesh_RenderDataFixup, STATGROUP_LoadTime);

/** Package name, that if set will cause only static meshes in that package to be rebuilt based on SM version. */
ENGINE_API FName GStaticMeshPackageNameToRebuild = NAME_None;

#if WITH_EDITORONLY_DATA
int32 GUpdateMeshLODGroupSettingsAtLoad = 0;
static FAutoConsoleVariableRef CVarStaticMeshUpdateMeshLODGroupSettingsAtLoad(
	TEXT("r.StaticMesh.UpdateMeshLODGroupSettingsAtLoad"),
	GUpdateMeshLODGroupSettingsAtLoad,
	TEXT("If set, LODGroup settings for static meshes will be applied at load time."));
#endif

static TAutoConsoleVariable<int32> CVarStripMinLodDataDuringCooking(
	TEXT("r.StaticMesh.StripMinLodDataDuringCooking"),
	0,
	TEXT("If non-zero, data for Static Mesh LOD levels below MinLOD will be discarded at cook time"));

static TAutoConsoleVariable<int32> CVarStaticMeshKeepMobileMinLODSettingOnDesktop(
	TEXT("r.StaticMesh.KeepMobileMinLODSettingOnDesktop"),
	0,
	TEXT("If non-zero, mobile setting for MinLOD will be stored in the cooked data for desktop platforms"));

static TAutoConsoleVariable<int32> CVarSupportDepthOnlyIndexBuffers(
	TEXT("r.SupportDepthOnlyIndexBuffers"),
	0,
	TEXT("Enables depth-only index buffers. Saves a little time at the expense of doubling the size of index buffers."),
	ECVF_ReadOnly | ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarSupportReversedIndexBuffers(
	TEXT("r.SupportReversedIndexBuffers"),
	0,
	TEXT("Enables reversed index buffers. Saves a little time at the expense of doubling the size of index buffers."),
	ECVF_ReadOnly | ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarStripDistanceFieldDataDuringLoad(
	TEXT("r.StaticMesh.StripDistanceFieldDataDuringLoad"),
	0,
	TEXT("If non-zero, data for distance fields will be discarded on load. TODO: change to discard during cook!."),
	ECVF_ReadOnly | ECVF_RenderThreadSafe);

const TCHAR* GMinLodQualityLevelCVarName = TEXT("r.StaticMesh.MinLodQualityLevel");
const TCHAR* GMinLodQualityLevelScalabilitySection = TEXT("ViewDistanceQuality");
int32 GMinLodQualityLevel = -1;
static FAutoConsoleVariableRef CVarStaticMeshMinLodQualityLevel(
	GMinLodQualityLevelCVarName,
	GMinLodQualityLevel,
	TEXT("The quality level for the Min stripping LOD. \n"),
	FConsoleVariableDelegate::CreateStatic(&UStaticMesh::OnLodStrippingQualityLevelChanged),
	ECVF_Scalability);

static TAutoConsoleVariable<int32> CVarForceEnableNaniteMeshes(
	TEXT("r.Nanite.ForceEnableMeshes"),
	0,
	TEXT("Force enables all meshes to also build Nanite data, regardless of the enabled flag on the asset."),
	ECVF_ReadOnly | ECVF_RenderThreadSafe);

static TAutoConsoleVariable<bool> CVarStaticMeshDefaultMeshPaintTextureSupport(
	TEXT("r.StaticMesh.DefaultMeshPaintTextureSupport"),
	true,
	TEXT("Default setting for whether static mesh assets support mesh paint textures."),
	ECVF_Default);

static TAutoConsoleVariable<bool> CVarStaticMeshRayTracingProxies(
	TEXT("r.StaticMesh.RayTracingProxies"),
	true,
	TEXT("Whether to use dedicated ray tracing LODs."),
	ECVF_ReadOnly);

static TAutoConsoleVariable<int32> CVarStaticMeshRayTracingProxiesLOD1MaxNumOfTriangles(
	TEXT("r.StaticMesh.RayTracingProxies.LOD1MaxNumOfTriangles"),
	64,
	TEXT("Maximum number of triangles used for Ray Tracing Proxy LOD 1."),
	ECVF_ReadOnly);

static TAutoConsoleVariable<bool> CVarNaniteVoxelsUseSourceMeshUVDensity(
	TEXT("r.Nanite.Voxels.UseSourceMeshUVDensity"),
	1,
	TEXT("Use source mesh data when calculating UV densities for Nanite meshes that simplify into voxels."),
	ECVF_ReadOnly);

#if ENABLE_COOK_STATS
namespace StaticMeshCookStats
{
	static FCookStats::FDDCResourceUsageStats UsageStats;
	static FCookStatsManager::FAutoRegisterCallback RegisterCookStats([](FCookStatsManager::AddStatFuncRef AddStat)
	{
		UsageStats.LogStats(AddStat, TEXT("StaticMesh.Usage"), TEXT(""));
	});
}
#endif

#if WITH_EDITOR

static bool ShouldStripNaniteFallbackMesh(const ITargetPlatform* TargetPlatform, const UStaticMesh* StaticMesh)
{
	return DoesTargetPlatformSupportNanite(TargetPlatform)
		&& TargetPlatform->ShouldStripNaniteFallbackMeshes()
		&& StaticMesh->GetNaniteSettings().GenerateFallback != ENaniteGenerateFallback::Enabled
		&& StaticMesh->HasValidNaniteData()
		&& !StaticMesh->bAllowCPUAccess
		&& !TargetPlatform->HasEditorOnlyData();
}

static void FillMaterialName(const TArray<FStaticMaterial>& StaticMaterials, TMap<int32, FName>& OutMaterialMap)
{
	OutMaterialMap.Empty(StaticMaterials.Num());

	for (int32 MaterialIndex = 0; MaterialIndex < StaticMaterials.Num(); ++MaterialIndex)
	{
		FName MaterialName = StaticMaterials[MaterialIndex].ImportedMaterialSlotName;
		if (MaterialName == NAME_None)
		{
			MaterialName = *(TEXT("MaterialSlot_") + FString::FromInt(MaterialIndex));
		}
		OutMaterialMap.Add(MaterialIndex, MaterialName);
	}
}

bool ShouldUseSourceMeshForUVDensity(const FMeshNaniteSettings& NaniteSettings)
{
	return NaniteSettings.bEnabled &&
		NaniteSettings.ShapePreservation == ENaniteShapePreservation::Voxelize &&
		CVarNaniteVoxelsUseSourceMeshUVDensity.GetValueOnAnyThread();
}

void ShowNaniteAssembliesDisabledWarningNotification()
{
	static std::atomic<bool> bSuppressNotification = false;
	if (bSuppressNotification.load())
	{
		return;
	}

	if (!IsInGameThread())
	{
		// Show the notification on the game thread only
		AsyncTask(ENamedThreads::GameThread, ShowNaniteAssembliesDisabledWarningNotification);
		return;
	}

	static const FText NaniteAssembliesWarningText = LOCTEXT("NaniteAssembliesDisabledWarning",
		"A Nanite Assembly asset was encountered while Nanite Assemblies are currently disabled. It may not appear as expected. "
		"Enable Nanite Foliage in Project Settings to enable support for Nanite Assemblies.");
	static const FText DontShowAgainMessageText = LOCTEXT("NaniteAssembliesEnabledWarningDoNotShow", "Don't show this again");
	static const FText DismissButtonText = LOCTEXT("NaniteAssembliesEnabledWarningDismiss", "Dismiss");
	static TWeakPtr<SNotificationItem> CurrentNotification;

	if (CurrentNotification.IsValid())
	{
		return;
	}

	FNotificationInfo Info(NaniteAssembliesWarningText);
	Info.Image = FAppStyle::GetBrush("Icons.WarningWithColor");
	Info.bFireAndForget = false;
	Info.CheckBoxText = DontShowAgainMessageText;
	Info.CheckBoxState = ECheckBoxState::Unchecked;
	Info.CheckBoxStateChanged = FOnCheckStateChanged::CreateLambda(
		[&](ECheckBoxState CheckboxValue)
		{
			bSuppressNotification.store(CheckboxValue == ECheckBoxState::Checked);
		}
	);
	Info.HyperlinkText = LOCTEXT("ProjectSettingsHyperlinkText", "Open Project Settings");
	Info.Hyperlink = FSimpleDelegate::CreateLambda(
		[]()
		{
			FGlobalTabmanager::Get()->TryInvokeTab(FName("ProjectSettings"));
		}
	);
	Info.ButtonDetails.Emplace(
		DismissButtonText,
		FText::GetEmpty(),
		FSimpleDelegate::CreateLambda(
			[&]()
			{
				if (TSharedPtr<SNotificationItem> Notification = CurrentNotification.Pin())
				{
					Notification->ExpireAndFadeout();
					CurrentNotification.Reset();
				}
			}
		),
		SNotificationItem::CS_None
	);
	CurrentNotification = FSlateNotificationManager::Get().AddNotification(Info);
}

#endif // WITH_EDITOR

namespace UE::StaticMesh::Private::Constants
{
	// Default Highest LOD Screen Size
	constexpr float LOD0ScreenSize = 2.0f;

	// Arbitrary constant used as a base in Pow(K, LODIndex) that achieves much the same progression as a
	// conversion of the old 1 / (MaxLODs * LODIndex) passed through the newer bounds computation.
	// i.e. this achieves much the same results, but is still fairly arbitrary.
	constexpr float AutoComputeLODPowerBase = 0.75f;

	constexpr float Tolerance = 0.01f;
}

/*-----------------------------------------------------------------------------
	FStaticMeshAsyncBuildWorker
-----------------------------------------------------------------------------*/

#if WITH_EDITOR

void FStaticMeshAsyncBuildWorker::DoWork()
{
	FStaticMeshAsyncBuildScope AsyncBuildScope(StaticMesh);
	UE_LOG_CONTEXT("StaticMesh", FSoftObjectPath(StaticMesh));

	if (PostLoadContext.IsValid())
	{
		UE::Tasks::FCancellationTokenScope CancellationScope(PostLoadContext->CancellationToken);
		StaticMesh->ExecutePostLoadInternal(*PostLoadContext);
	}

	if (BuildContext.IsValid())
	{
		UE::Tasks::FCancellationTokenScope CancellationScope(BuildContext->CancellationToken);
		BuildContext->bHasRenderDataChanged = StaticMesh->ExecuteBuildInternal(BuildContext->BuildParameters);
	}
}

void FStaticMeshAsyncBuildWorker::MarkAsCanceled()
{
	if (FStaticMeshCompilingManager::Get().IsAsyncCompilationCancelable())
	{
	    TRACE_CPUPROFILER_EVENT_SCOPE(FStaticMeshAsyncBuildWorker::MarkAsCanceled);
    
	    if (PostLoadContext.IsValid())
	    {
		    PostLoadContext->CancellationToken.Cancel();
	    }
    
	    if (BuildContext.IsValid())
	    {
		    BuildContext->CancellationToken.Cancel();
	    }
	}
}

#endif // #if WITH_EDITOR

/*-----------------------------------------------------------------------------
	FStaticMeshSectionAreaWeightedTriangleSamplerBuffer
-----------------------------------------------------------------------------*/

FStaticMeshSectionAreaWeightedTriangleSamplerBuffer::FStaticMeshSectionAreaWeightedTriangleSamplerBuffer()
{
}

FStaticMeshSectionAreaWeightedTriangleSamplerBuffer::~FStaticMeshSectionAreaWeightedTriangleSamplerBuffer()
{
}

void FStaticMeshSectionAreaWeightedTriangleSamplerBuffer::InitRHI(FRHICommandListBase& RHICmdList)
{
	ReleaseRHI();

	if (Samplers && Samplers->Num() > 0)
	{
		// Count triangle count for all sections and required memory
		const uint32 AllSectionCount = Samplers->Num();
		uint32 TriangleCount = 0;
		for (uint32 i = 0; i < AllSectionCount; ++i)
		{
			TriangleCount += (*Samplers)[i].GetNumEntries();
		}
		const uint32 SizeByte = TriangleCount * sizeof(SectionTriangleInfo);

		const FRHIBufferCreateDesc Desc =
			FRHIBufferCreateDesc::CreateVertex<SectionTriangleInfo>(TEXT("StaticMeshSectionAreaWeightedTriangleSamplerBuffer"), TriangleCount)
			.AddUsage(BUF_Static | BUF_ShaderResource)
			.SetInitialState(ERHIAccess::VertexOrIndexBuffer | ERHIAccess::SRVMask)
			.SetInitActionInitializer();

		TRHIBufferInitializer<SectionTriangleInfo> Initializer = RHICmdList.CreateBufferInitializer(Desc);

		// Now compute the alias look up table for unifor; distribution for all section and all triangles
		for (uint32 SectionIndex = 0; SectionIndex < AllSectionCount; SectionIndex++)
		{
			const FStaticMeshSectionAreaWeightedTriangleSampler& TriangleSampler = (*Samplers)[SectionIndex];
			TConstArrayView<float> ProbTris = TriangleSampler.GetProb();
			TConstArrayView<int32> AliasTris = TriangleSampler.GetAlias();
			const uint32 NumTriangle = TriangleSampler.GetNumEntries();

			for (uint32 TriangleIndex = 0; TriangleIndex < NumTriangle; TriangleIndex++)
			{
				SectionTriangleInfo NewTriangleInfo = { ProbTris[TriangleIndex], (uint32)AliasTris[TriangleIndex] };
				Initializer[TriangleIndex] = NewTriangleInfo;
			}
		}
		BufferSectionTriangleRHI = Initializer.Finalize();
		BufferSectionTriangleSRV = RHICmdList.CreateShaderResourceView(
			BufferSectionTriangleRHI, 
			FRHIViewDesc::CreateBufferSRV()
				.SetType(FRHIViewDesc::EBufferType::Typed)
				.SetFormat(PF_R32G32_UINT));
	}
}

void FStaticMeshSectionAreaWeightedTriangleSamplerBuffer::ReleaseRHI()
{
	BufferSectionTriangleSRV.SafeRelease();
	BufferSectionTriangleRHI.SafeRelease();
}


/*-----------------------------------------------------------------------------
	FStaticMeshLODResources
-----------------------------------------------------------------------------*/

FArchive& operator<<(FArchive& Ar, FStaticMeshSection& Section)
{
	// Note: this is all derived data, native versioning is not needed, but be sure to bump STATICMESH_DERIVEDDATA_VER when modifying!
	Ar << Section.MaterialIndex;
	Ar << Section.FirstIndex;
	Ar << Section.NumTriangles;
	Ar << Section.MinVertexIndex;
	Ar << Section.MaxVertexIndex;
	Ar << Section.bEnableCollision;
	Ar << Section.bCastShadow;

	if(
		!(Ar.IsLoading() && 
		  Ar.CustomVer(FRenderingObjectVersion::GUID) < FRenderingObjectVersion::StaticMeshSectionForceOpaqueField)
	  )
	{
		Ar << Section.bForceOpaque;
	}

#if WITH_EDITORONLY_DATA
	if((!Ar.IsCooking() && !Ar.IsFilterEditorOnly()) || (Ar.IsCooking() && Ar.CookingTarget()->HasEditorOnlyData()))
	{
		for (int32 UVIndex = 0; UVIndex < MAX_STATIC_TEXCOORDS; ++UVIndex)
		{
			Ar << Section.UVDensities[UVIndex];
			Ar << Section.Weights[UVIndex];
		}
	}
#endif

	Ar << Section.bVisibleInRayTracing;
	Ar << Section.bAffectDistanceFieldLighting;


	return Ar;
}

int32 FStaticMeshLODResources::GetPlatformMinLODIdx(const ITargetPlatform* TargetPlatform, UStaticMesh* StaticMesh)
{
#if WITH_EDITOR
	check(TargetPlatform && StaticMesh);
	if (StaticMesh->IsMinLodQualityLevelEnable())
	{
		// get all supported quality level from scalability + engine ini files
		return StaticMesh->GetQualityLevelMinLOD().GetValueForPlatform(TargetPlatform);
	}
	else
	{
		return StaticMesh->GetMinLOD().GetValueForPlatform(*TargetPlatform->IniPlatformName());
	}
	
#else
	return 0;
#endif
}

uint8 FStaticMeshLODResources::GenerateClassStripFlags(FArchive& Ar, UStaticMesh* OwnerStaticMesh, int32 Index)
{
#if WITH_EDITOR
	// Defined class flags for possible stripping
	const uint8 MinLodDataStripFlag = CDSF_MinLodData;
	const uint8 ReversedIndexBufferStripFlag = CDSF_ReversedIndexBuffer;
	const uint8 RayTracingResourcesFlag = CDSF_RayTracingResources;

	const bool bWantToStripLOD = Ar.IsCooking()
		&& (CVarStripMinLodDataDuringCooking.GetValueOnAnyThread() != 0)
		&& OwnerStaticMesh
		&& GetPlatformMinLODIdx(Ar.CookingTarget(), OwnerStaticMesh) > Index;
	const bool bSupportRayTracing = OwnerStaticMesh ? OwnerStaticMesh->bSupportRayTracing : false;
	const bool bWantToStripRayTracingResources = Ar.IsCooking() && (!Ar.CookingTarget()->UsesRayTracing() || !bSupportRayTracing);

	return
		(bWantToStripLOD ? MinLodDataStripFlag : 0) |
		(bWantToStripRayTracingResources ? RayTracingResourcesFlag : 0);
#else
	return 0;
#endif
}

bool FStaticMeshLODResources::IsLODCookedOut(const ITargetPlatform* TargetPlatform, UStaticMesh* StaticMesh, bool bIsBelowMinLOD)
{
	check(StaticMesh);
#if WITH_EDITOR
	if (TargetPlatform && ShouldStripNaniteFallbackMesh(TargetPlatform, StaticMesh))
	{
		return true;
	}

	if (!bIsBelowMinLOD)
	{
		return false;
	}

	if (!TargetPlatform)
	{
		TargetPlatform = GetTargetPlatformManagerRef().GetRunningTargetPlatform();
	}
	check(TargetPlatform);

	return !StaticMesh->GetEnableLODStreaming(TargetPlatform);
#else
	return false;
#endif
}

bool FStaticMeshLODResources::IsLODInlined(const ITargetPlatform* TargetPlatform, UStaticMesh* StaticMesh, int32 LODIdx, bool bIsBelowMinLOD)
{
	check(StaticMesh);
#if WITH_EDITOR
	if (!TargetPlatform)
	{
		TargetPlatform = GetTargetPlatformManagerRef().GetRunningTargetPlatform();
	}
	check(TargetPlatform);

	if (!StaticMesh->GetEnableLODStreaming(TargetPlatform))
	{
		return true;
	}

	if (bIsBelowMinLOD)
	{
		return false;
	}

	int32 MaxNumStreamedLODs = 0;
	const int32 NumStreamedLODsOverride = StaticMesh->GetNumStreamedLODs().GetValueForPlatform(*TargetPlatform->IniPlatformName());
	if (NumStreamedLODsOverride >= 0)
	{
		MaxNumStreamedLODs = NumStreamedLODsOverride;
	}
	else
	{
		const FStaticMeshLODGroup& LODGroupSettings = TargetPlatform->GetStaticMeshLODSettings().GetLODGroup(StaticMesh->GetLODGroup());
		MaxNumStreamedLODs = LODGroupSettings.GetDefaultMaxNumStreamedLODs();
	}
	
	FStaticMeshRenderData& PlatformRenderData = UStaticMesh::GetPlatformStaticMeshRenderData(StaticMesh, TargetPlatform);
	const int32 NumLODs = PlatformRenderData.LODResources.Num();
	const int32 NumStreamedLODs = FMath::Min(MaxNumStreamedLODs, NumLODs - 1);
	const int32 InlinedLODStartIdx = NumStreamedLODs;
	return LODIdx >= InlinedLODStartIdx;
#else
	return false;
#endif
}

int32 FStaticMeshLODResources::GetNumOptionalLODsAllowed(const ITargetPlatform* TargetPlatform, UStaticMesh* StaticMesh)
{
#if WITH_EDITOR
	check(TargetPlatform && StaticMesh);
	const FStaticMeshLODGroup& LODGroupSettings = TargetPlatform->GetStaticMeshLODSettings().GetLODGroup(StaticMesh->GetLODGroup());
	return LODGroupSettings.GetDefaultMaxNumOptionalLODs();
#else
	return 0;
#endif
}

void FStaticMeshLODResources::AccumVertexBuffersSize(const FStaticMeshVertexBuffers& VertexBuffers, uint32& OutSize)
{
#if (WITH_EDITOR || DO_CHECK)
	const FPositionVertexBuffer& Pos = VertexBuffers.PositionVertexBuffer;
	const FStaticMeshVertexBuffer& TanTex = VertexBuffers.StaticMeshVertexBuffer;
	const FColorVertexBuffer& Color = VertexBuffers.ColorVertexBuffer;
	OutSize += Pos.GetNumVertices() * Pos.GetStride();
	OutSize += TanTex.GetResourceSize();
	OutSize += Color.GetNumVertices() * Color.GetStride();
#endif
}

void FStaticMeshLODResources::AccumIndexBufferSize(const FRawStaticIndexBuffer& IndexBuffer, uint32& OutSize)
{
#if (WITH_EDITOR || DO_CHECK)
	OutSize += IndexBuffer.GetIndexDataSize();
#endif
}

void FStaticMeshLODResources::AccumRayTracingGeometrySize(const FRayTracingGeometry& RayTracingGeometry, uint32& OutSize)
{
#if (WITH_EDITOR || DO_CHECK)
	OutSize += RayTracingGeometry.RawData.Num();
#endif
}

uint32 FStaticMeshLODResources::FStaticMeshBuffersSize::CalcBuffersSize() const
{
	// Assumes these two cvars don't change at runtime
	const bool bEnableDepthOnlyIndexBuffer = !!CVarSupportDepthOnlyIndexBuffers.GetValueOnAnyThread();
	const bool bEnableReversedIndexBuffer = !!CVarSupportReversedIndexBuffers.GetValueOnAnyThread();
	return SerializedBuffersSize
		- (bEnableDepthOnlyIndexBuffer ? 0 : DepthOnlyIBSize)
		- (bEnableReversedIndexBuffer ? 0 : ReversedIBsSize);
}

void FStaticMeshLODResources::SerializeBuffers(FArchive& Ar, UStaticMesh* OwnerStaticMesh, uint8 InStripFlags, FStaticMeshBuffersSize& OutBuffersSize)
{
	Ar.UsingCustomVersion(FUE5ReleaseStreamObjectVersion::GUID);

	// If the index buffers have already been initialized, do not change the behavior since the RHI resource pointer may have been cached somewhere already.
	const bool bEnableDepthOnlyIndexBuffer = bHasDepthOnlyIndices || !!CVarSupportDepthOnlyIndexBuffers.GetValueOnAnyThread();
	const bool bEnableReversedIndexBuffer = bHasReversedIndices || bHasReversedDepthOnlyIndices || !!CVarSupportReversedIndexBuffers.GetValueOnAnyThread();

	// See if the mesh wants to keep resources CPU accessible
	bool bMeshCPUAcces = OwnerStaticMesh ? OwnerStaticMesh->bAllowCPUAccess : false;

	// Note: this is all derived data, native versioning is not needed, but be sure to bump STATICMESH_DERIVEDDATA_VER when modifying!

	// On cooked platforms we never need the resource data.
	// TODO: Not needed in uncooked games either after PostLoad!
	bool bNeedsCPUAccess = !FPlatformProperties::RequiresCookedData() || bMeshCPUAcces;

	if (FPlatformProperties::RequiresCookedData())
	{
		if (bNeedsCPUAccess && OwnerStaticMesh)
		{
			UE_LOGF(LogStaticMesh, Log, "[%ls] Mesh is marked for CPU read.", *OwnerStaticMesh->GetName());
		}
	}

	FStripDataFlags StripFlags(Ar, InStripFlags);

	VertexBuffers.PositionVertexBuffer.Serialize(Ar, bNeedsCPUAccess);
	VertexBuffers.StaticMeshVertexBuffer.Serialize(Ar, bNeedsCPUAccess);
	VertexBuffers.ColorVertexBuffer.Serialize(Ar, bNeedsCPUAccess);
	OutBuffersSize.Clear();
	AccumVertexBuffersSize(VertexBuffers, OutBuffersSize.SerializedBuffersSize);

	IndexBuffer.Serialize(Ar, bNeedsCPUAccess);
	AccumIndexBufferSize(IndexBuffer, OutBuffersSize.SerializedBuffersSize);

	const bool bSerializeReversedIndexBuffer = !StripFlags.IsClassDataStripped(CDSF_ReversedIndexBuffer);
	const bool bSerializeWireframeIndexBuffer = !StripFlags.IsEditorDataStripped();
	const bool bSerializeRayTracingGeometry = !StripFlags.IsClassDataStripped(CDSF_RayTracingResources);

	if ((bSerializeReversedIndexBuffer && bEnableReversedIndexBuffer) || bSerializeWireframeIndexBuffer)
	{
		if (AdditionalIndexBuffers == nullptr)
		{
			AdditionalIndexBuffers = new FAdditionalStaticMeshIndexBuffers();
		}
	}

	// If the reversed index buffer feature is disabled, we serialize with dummy buffers (writes empty result if saving, reads and discards if loading).
	// The separate structure is used to avoid the need to dynamically allocate AdditionalIndexBuffers to hold a result we are discarding.
	FAdditionalStaticMeshIndexBuffers DummyBuffers;
	FAdditionalStaticMeshIndexBuffers* ReversedAdditionalIndexBuffers = bEnableReversedIndexBuffer ? AdditionalIndexBuffers : &DummyBuffers;

	if (bSerializeReversedIndexBuffer)
	{
		ReversedAdditionalIndexBuffers->ReversedIndexBuffer.Serialize(Ar, bNeedsCPUAccess);
		AccumIndexBufferSize(ReversedAdditionalIndexBuffers->ReversedIndexBuffer, OutBuffersSize.ReversedIBsSize);
		AccumIndexBufferSize(ReversedAdditionalIndexBuffers->ReversedIndexBuffer, OutBuffersSize.SerializedBuffersSize);
	}

	DepthOnlyIndexBuffer.Serialize(Ar, bNeedsCPUAccess);
	AccumIndexBufferSize(DepthOnlyIndexBuffer, OutBuffersSize.DepthOnlyIBSize);
	AccumIndexBufferSize(DepthOnlyIndexBuffer, OutBuffersSize.SerializedBuffersSize);
	if (!bEnableDepthOnlyIndexBuffer)
	{
		DepthOnlyIndexBuffer.Discard();
		DepthOnlyIndexBuffer.ClearMetaData();
	}

	if (bSerializeReversedIndexBuffer)
	{
		ReversedAdditionalIndexBuffers->ReversedDepthOnlyIndexBuffer.Serialize(Ar, bNeedsCPUAccess);
		AccumIndexBufferSize(ReversedAdditionalIndexBuffers->ReversedDepthOnlyIndexBuffer, OutBuffersSize.ReversedIBsSize);
		AccumIndexBufferSize(ReversedAdditionalIndexBuffers->ReversedDepthOnlyIndexBuffer, OutBuffersSize.SerializedBuffersSize);
		if (!bEnableDepthOnlyIndexBuffer)
		{
			ReversedAdditionalIndexBuffers->ReversedDepthOnlyIndexBuffer.Discard();
			ReversedAdditionalIndexBuffers->ReversedDepthOnlyIndexBuffer.ClearMetaData();
		}
	}

	if (bSerializeWireframeIndexBuffer)
	{
		AdditionalIndexBuffers->WireframeIndexBuffer.Serialize(Ar, bNeedsCPUAccess);
		AccumIndexBufferSize(AdditionalIndexBuffers->WireframeIndexBuffer, OutBuffersSize.SerializedBuffersSize);
	}

	if (Ar.IsLoading() && Ar.CustomVer(FUE5ReleaseStreamObjectVersion::GUID) < FUE5ReleaseStreamObjectVersion::RemovingTessellation && !StripFlags.IsClassDataStripped(CDSF_AdjacencyData_DEPRECATED))
	{
		FRawStaticIndexBuffer AdjacencyIndexBuffer;
		AdjacencyIndexBuffer.Serialize(Ar, bNeedsCPUAccess);
	}

	if (bSerializeRayTracingGeometry)
	{
		FRayTracingGeometry DummyRayTracingGeometry;
		FRayTracingGeometry* SerializedRayTracingGeometry = RayTracingGeometry != nullptr ? RayTracingGeometry : &DummyRayTracingGeometry;

		Ar << SerializedRayTracingGeometry->RawDataHeader;
		SerializedRayTracingGeometry->RawData.BulkSerialize(Ar);
		AccumRayTracingGeometrySize(*SerializedRayTracingGeometry, OutBuffersSize.SerializedBuffersSize);

		if (Ar.IsLoading() && RayTracing::ShouldForceRuntimeBLAS())
		{
			SerializedRayTracingGeometry->RawDataHeader = FRayTracingGeometryOfflineDataHeader();
			SerializedRayTracingGeometry->RawData.Discard();
		}
	}

	AreaWeightedSectionSamplers.SetNum(Sections.Num());
	for (FStaticMeshSectionAreaWeightedTriangleSampler& Sampler : AreaWeightedSectionSamplers)
	{
		Sampler.Serialize(Ar);
	}
	AreaWeightedSampler.Serialize(Ar);

	// Update metadata but only if serialization was successful. This needs to be done now because on cooked platform, indices are discarded after RHIInit.
	if (!Ar.IsError())
	{
		bHasWireframeIndices = AdditionalIndexBuffers && bSerializeWireframeIndexBuffer && AdditionalIndexBuffers->WireframeIndexBuffer.GetNumIndices() != 0;
		bHasDepthOnlyIndices = DepthOnlyIndexBuffer.GetNumIndices() != 0;
		bHasReversedIndices = AdditionalIndexBuffers && bSerializeReversedIndexBuffer && AdditionalIndexBuffers->ReversedIndexBuffer.GetNumIndices() != 0;
		bHasReversedDepthOnlyIndices = AdditionalIndexBuffers && bSerializeReversedIndexBuffer && AdditionalIndexBuffers->ReversedDepthOnlyIndexBuffer.GetNumIndices() != 0;
		bHasColorVertexData = VertexBuffers.ColorVertexBuffer.GetNumVertices() > 0;
		DepthOnlyNumTriangles = DepthOnlyIndexBuffer.GetNumIndices() / 3;
	}
}

void FStaticMeshLODResources::SerializeAvailabilityInfo(FArchive& Ar)
{
	Ar.UsingCustomVersion(FUE5ReleaseStreamObjectVersion::GUID);
	bool bHasAdjacencyInfo = false;
	const bool bEnableDepthOnlyIndexBuffer = !!CVarSupportDepthOnlyIndexBuffers.GetValueOnAnyThread();
	const bool bEnableReversedIndexBuffer = !!CVarSupportReversedIndexBuffers.GetValueOnAnyThread();

	Ar << DepthOnlyNumTriangles;
	uint32 Packed;
#if WITH_EDITOR
	if (Ar.IsSaving())
	{
		Packed = (bHasDepthOnlyIndices << 1u)
			| (bHasReversedIndices << 2u)
			| (bHasReversedDepthOnlyIndices << 3u)
			| (bHasColorVertexData << 4u)
			| (bHasWireframeIndices << 5u);
		Ar << Packed;
	}
	else
#endif
	{
		Ar << Packed;
		DepthOnlyNumTriangles *= static_cast<uint32>(bEnableDepthOnlyIndexBuffer);
		bHasAdjacencyInfo = Packed & 1u;
		bHasDepthOnlyIndices = bEnableDepthOnlyIndexBuffer && !!(Packed & 2u);
		bHasReversedIndices = bEnableReversedIndexBuffer && !!(Packed & 4u);
		bHasReversedDepthOnlyIndices = bEnableReversedIndexBuffer && !!(Packed & 8u);
		bHasColorVertexData = (Packed >> 4u) & 1u;
		bHasWireframeIndices = (Packed >> 5u) & 1u;
	}

	VertexBuffers.StaticMeshVertexBuffer.SerializeMetaData(Ar);
	VertexBuffers.PositionVertexBuffer.SerializeMetaData(Ar);
	VertexBuffers.ColorVertexBuffer.SerializeMetaData(Ar);
	IndexBuffer.SerializeMetaData(Ar);

	if ((bHasReversedIndices && bEnableReversedIndexBuffer) || bHasWireframeIndices)
	{
		if (AdditionalIndexBuffers == nullptr)
		{
			AdditionalIndexBuffers = new FAdditionalStaticMeshIndexBuffers();
		}
	}

	// If the reversed index buffer or wireframe features are disabled, we serialize with dummy buffers.
	FAdditionalStaticMeshIndexBuffers DummyBuffers;
	FAdditionalStaticMeshIndexBuffers* ReversedAdditionalIndexBuffers = bHasReversedIndices && bEnableReversedIndexBuffer ? AdditionalIndexBuffers : &DummyBuffers;
	FAdditionalStaticMeshIndexBuffers* WireframeAdditionalIndexBuffers = bHasWireframeIndices ? AdditionalIndexBuffers : &DummyBuffers;

	ReversedAdditionalIndexBuffers->ReversedIndexBuffer.SerializeMetaData(Ar);
	if (!bHasReversedIndices && AdditionalIndexBuffers)
	{
		// Reversed indices are either stripped during cook or will be stripped on load.
		// In either case, clear CachedNumIndices to show that the buffer will be empty after actual loading
		AdditionalIndexBuffers->ReversedIndexBuffer.Discard();
		AdditionalIndexBuffers->ReversedIndexBuffer.ClearMetaData();
	}
	DepthOnlyIndexBuffer.SerializeMetaData(Ar);
	if (!bHasDepthOnlyIndices)
	{
		DepthOnlyIndexBuffer.Discard();
		DepthOnlyIndexBuffer.ClearMetaData();
	}
	ReversedAdditionalIndexBuffers->ReversedDepthOnlyIndexBuffer.SerializeMetaData(Ar);
	if (!bHasReversedDepthOnlyIndices && AdditionalIndexBuffers)
	{
		AdditionalIndexBuffers->ReversedDepthOnlyIndexBuffer.Discard();
		AdditionalIndexBuffers->ReversedDepthOnlyIndexBuffer.ClearMetaData();
	}
	WireframeAdditionalIndexBuffers->WireframeIndexBuffer.SerializeMetaData(Ar);
	if (!bHasWireframeIndices && AdditionalIndexBuffers)
	{
		AdditionalIndexBuffers->WireframeIndexBuffer.Discard();
		AdditionalIndexBuffers->WireframeIndexBuffer.ClearMetaData();
	}
	if (Ar.IsLoading() && Ar.CustomVer(FUE5ReleaseStreamObjectVersion::GUID) < FUE5ReleaseStreamObjectVersion::RemovingTessellation)
	{
		FRawStaticIndexBuffer AdjacencyIndexBuffer;
		AdjacencyIndexBuffer.SerializeMetaData(Ar);
	}

	{
		FRayTracingGeometry DummyRayTracingGeometry;
		FRayTracingGeometry* SerializedRayTracingGeometry = RayTracingGeometry != nullptr ? RayTracingGeometry : &DummyRayTracingGeometry;
		Ar << SerializedRayTracingGeometry->RawDataHeader;

		if (Ar.IsLoading() && RayTracing::ShouldForceRuntimeBLAS())
		{
			SerializedRayTracingGeometry->RawDataHeader = FRayTracingGeometryOfflineDataHeader();
		}
	}
}

void FStaticMeshLODResources::ClearAvailabilityInfo()
{
	DepthOnlyNumTriangles = 0;
	bHasDepthOnlyIndices = false;
	bHasReversedIndices = false;
	bHasReversedDepthOnlyIndices = false;
	bHasColorVertexData = false;
	bHasWireframeIndices = false;
	VertexBuffers.StaticMeshVertexBuffer.ClearMetaData();
	VertexBuffers.PositionVertexBuffer.ClearMetaData();
	VertexBuffers.ColorVertexBuffer.ClearMetaData();

	IndexBuffer.ClearMetaData();
	IndexBuffer.ClearMetaData();
	DepthOnlyIndexBuffer.ClearMetaData();

	if (AdditionalIndexBuffers)
	{
		AdditionalIndexBuffers->ReversedIndexBuffer.ClearMetaData();
		AdditionalIndexBuffers->ReversedDepthOnlyIndexBuffer.ClearMetaData();
		AdditionalIndexBuffers->WireframeIndexBuffer.ClearMetaData();
	}

	delete AdditionalIndexBuffers;
	AdditionalIndexBuffers = nullptr;
}

void FStaticMeshLODResources::Serialize(FArchive& Ar, UObject* Owner, int32 Index)
{
	DECLARE_SCOPE_CYCLE_COUNTER(TEXT("FStaticMeshLODResources::Serialize"), STAT_StaticMeshLODResources_Serialize, STATGROUP_LoadTime);

	bool bUsingCookedEditorData = false;
#if WITH_EDITORONLY_DATA
	bUsingCookedEditorData = Owner->GetOutermost()->bIsCookedForEditor;
#endif

	UStaticMesh* OwnerStaticMesh = Cast<UStaticMesh>(Owner);
	// Actual flags used during serialization
	const uint8 ClassDataStripFlags = GenerateClassStripFlags(Ar, OwnerStaticMesh, Index);
	FStripDataFlags StripFlags(Ar, ClassDataStripFlags);

	Ar << Sections;
	Ar << SourceMeshBounds;
	Ar << MaxDeviation;

#if WITH_EDITORONLY_DATA
	if ((!Ar.IsCooking() && !Ar.IsFilterEditorOnly()) || (Ar.IsCooking() && Ar.CookingTarget()->HasEditorOnlyData()))
	{
		Ar << WedgeMap;
	}
#endif // #if WITH_EDITORONLY_DATA

#if WITH_EDITOR
	const bool bIsBelowMinLOD = StripFlags.IsClassDataStripped(CDSF_MinLodData)
		|| (Ar.IsCooking() && OwnerStaticMesh && Index < GetPlatformMinLODIdx(Ar.CookingTarget(), OwnerStaticMesh));
#else
	const bool bIsBelowMinLOD = false;
#endif
	bool bIsLODCookedOut = IsLODCookedOut(Ar.CookingTarget(), OwnerStaticMesh, bIsBelowMinLOD);
	Ar << bIsLODCookedOut;

	bool bInlined = bIsLODCookedOut || IsLODInlined(Ar.CookingTarget(), OwnerStaticMesh, Index, bIsBelowMinLOD);
	Ar << bInlined;
	bBuffersInlined = bInlined;

	if (!StripFlags.IsAudioVisualDataStripped() && !bIsLODCookedOut)
	{
		{
			bool bHasRayTracingGeometry = (RayTracingGeometry != nullptr);

#if WITH_EDITOR
			if (Ar.IsCooking() && !Ar.CookingTarget()->UsesRayTracing())
			{
				bHasRayTracingGeometry = false; // strip ray tracing geometry
			}
#endif

			Ar << bHasRayTracingGeometry;

			if (bHasRayTracingGeometry && RayTracingGeometry == nullptr)
			{
				checkf(Ar.IsLoading(), TEXT("bHasRayTracingGeometry unexpectedly changed even though we are not loading data."));
				RayTracingGeometry = new FRayTracingGeometry();
			}
		}

		FStaticMeshBuffersSize TmpBuffersSize;
		TArray<uint8> TmpBuff;

		if (bInlined)
		{
			SerializeBuffers(Ar, OwnerStaticMesh, ClassDataStripFlags, TmpBuffersSize);
			Ar << TmpBuffersSize;
			BuffersSize = TmpBuffersSize.CalcBuffersSize();
		}
		else if (FPlatformProperties::RequiresCookedData() || Ar.IsCooking() || bUsingCookedEditorData)
		{
#if WITH_EDITORONLY_DATA
			uint32 BulkDataSize = 0;
#endif

#if WITH_EDITOR
			if (Ar.IsSaving())
			{
				const int32 MaxNumOptionalLODs = GetNumOptionalLODsAllowed(Ar.CookingTarget(), OwnerStaticMesh);
				const int32 OptionalLODIdx = GetPlatformMinLODIdx(Ar.CookingTarget(), OwnerStaticMesh) - Index;
				const bool bDiscardBulkData = OptionalLODIdx > MaxNumOptionalLODs;

				if (!bDiscardBulkData)
				{
					FMemoryWriter MemWriter(TmpBuff, true);
					MemWriter.SetSavePackageData(Ar.GetSavePackageData());
					MemWriter.SetByteSwapping(Ar.IsByteSwapping());
					SerializeBuffers(MemWriter, OwnerStaticMesh, ClassDataStripFlags, TmpBuffersSize);
				}

				bIsOptionalLOD = bIsBelowMinLOD;
				const uint32 BulkDataFlags = (bDiscardBulkData ? 0 : BULKDATA_Force_NOT_InlinePayload)
					| (bIsOptionalLOD ? BULKDATA_OptionalPayload : 0);
				const uint32 OldBulkDataFlags = BulkData.GetBulkDataFlags();
				BulkData.ClearBulkDataFlags(0xffffffffu);
				BulkData.SetBulkDataFlags(BulkDataFlags);
				if (TmpBuff.Num() > 0)
				{
					BulkData.Lock(LOCK_READ_WRITE);
					void* BulkDataMem = BulkData.Realloc(TmpBuff.Num());
					FMemory::Memcpy(BulkDataMem, TmpBuff.GetData(), TmpBuff.Num());
					BulkData.Unlock();
				}
				BulkData.Serialize(Ar, Owner, Index);
				BulkData.ClearBulkDataFlags(0xffffffffu);
				BulkData.SetBulkDataFlags(OldBulkDataFlags);
			}
			else
#endif
			{
				StreamingBulkData.Serialize(Ar, Owner, Index, false);
				bIsOptionalLOD = StreamingBulkData.IsOptional();

#if WITH_EDITORONLY_DATA
				BulkDataSize = (uint32)StreamingBulkData.GetBulkDataSize();

				// Streaming CPU data in editor build isn't supported yet because tools and utils need access
				if (bUsingCookedEditorData && BulkDataSize > 0)
				{
					TmpBuff.Empty(BulkDataSize);
					TmpBuff.AddUninitialized(BulkDataSize);
					void* Dest = TmpBuff.GetData();
					StreamingBulkData.GetCopy(&Dest);
				}
#endif
			}

			SerializeAvailabilityInfo(Ar);

			Ar << TmpBuffersSize;
			BuffersSize = TmpBuffersSize.CalcBuffersSize();

			if (Ar.IsLoading() && bIsOptionalLOD)
			{
				ClearAvailabilityInfo();
			}

#if WITH_EDITORONLY_DATA
			if (Ar.IsLoading() && bUsingCookedEditorData && BulkDataSize > 0)
			{
				ClearAvailabilityInfo();
				FMemoryReader MemReader(TmpBuff, true);
				MemReader.SetByteSwapping(Ar.IsByteSwapping());
				SerializeBuffers(MemReader, OwnerStaticMesh, ClassDataStripFlags, TmpBuffersSize);
			}
#endif
		}
	}
}

void FStaticMeshLODResources::GetResourceSizeEx(FResourceSizeEx& CumulativeResourceSize) const
{
	CumulativeResourceSize.AddUnknownMemoryBytes(TEXT("TexcoordBuffer and TangentBuffer"), VertexBuffers.StaticMeshVertexBuffer.GetResourceSize());
	CumulativeResourceSize.AddUnknownMemoryBytes(TEXT("PositionVertexBuffer"), VertexBuffers.PositionVertexBuffer.GetStride() * VertexBuffers.PositionVertexBuffer.GetNumVertices());
	CumulativeResourceSize.AddUnknownMemoryBytes(TEXT("ColorVertexBuffer"), VertexBuffers.ColorVertexBuffer.GetStride() * VertexBuffers.ColorVertexBuffer.GetNumVertices());

	const int32 IndexStride = (IndexBuffer.Is32Bit() ? 4 : 2);
	CumulativeResourceSize.AddUnknownMemoryBytes(TEXT("IndexBuffer"), IndexBuffer.GetNumIndices() * IndexStride);
	CumulativeResourceSize.AddUnknownMemoryBytes(TEXT("DepthOnlyIndexBuffer"), DepthOnlyIndexBuffer.GetNumIndices() * IndexStride);

	if (AdditionalIndexBuffers)
	{
		CumulativeResourceSize.AddUnknownMemoryBytes(TEXT("ReversedDepthOnlyIndexBuffer"), AdditionalIndexBuffers->ReversedDepthOnlyIndexBuffer.GetNumIndices() * IndexStride);
		CumulativeResourceSize.AddUnknownMemoryBytes(TEXT("ReversedIndexBuffer"), AdditionalIndexBuffers->ReversedIndexBuffer.GetNumIndices() * IndexStride);
		CumulativeResourceSize.AddUnknownMemoryBytes(TEXT("WireframeIndexBuffer"), AdditionalIndexBuffers->WireframeIndexBuffer.GetNumIndices() * IndexStride);
	}

	CumulativeResourceSize.AddUnknownMemoryBytes(Sections.GetAllocatedSize());
}

SIZE_T FStaticMeshLODResources::GetCPUAccessMemoryOverhead() const
{
	int32 NumIndices = IndexBuffer.GetAllowCPUAccess() ? IndexBuffer.GetNumIndices() : 0;
	NumIndices += DepthOnlyIndexBuffer.GetAllowCPUAccess() ? DepthOnlyIndexBuffer.GetNumIndices() : 0;
	if (AdditionalIndexBuffers)
	{
		NumIndices += AdditionalIndexBuffers->ReversedDepthOnlyIndexBuffer.GetAllowCPUAccess() ? AdditionalIndexBuffers->ReversedDepthOnlyIndexBuffer.GetNumIndices() : 0;
		NumIndices += AdditionalIndexBuffers->ReversedIndexBuffer.GetAllowCPUAccess() ? AdditionalIndexBuffers->ReversedIndexBuffer.GetNumIndices() : 0;
		NumIndices += AdditionalIndexBuffers->WireframeIndexBuffer.GetAllowCPUAccess() ? AdditionalIndexBuffers->WireframeIndexBuffer.GetNumIndices() : 0;
	}
	return NumIndices * (IndexBuffer.Is32Bit() ? 4 : 2) +
		(VertexBuffers.StaticMeshVertexBuffer.GetAllowCPUAccess() ? VertexBuffers.StaticMeshVertexBuffer.GetResourceSize() : 0) +
		(VertexBuffers.PositionVertexBuffer.GetAllowCPUAccess() ? VertexBuffers.PositionVertexBuffer.GetStride() * VertexBuffers.PositionVertexBuffer.GetNumVertices() : 0) +
		(VertexBuffers.ColorVertexBuffer.GetAllowCPUAccess() ? VertexBuffers.ColorVertexBuffer.GetStride() * VertexBuffers.ColorVertexBuffer.GetNumVertices() : 0);
}

int32 FStaticMeshLODResources::GetNumTriangles() const
{
	int32 NumTriangles = 0;
	for(int32 SectionIndex = 0;SectionIndex < Sections.Num();SectionIndex++)
	{
		NumTriangles += Sections[SectionIndex].NumTriangles;
	}
	return NumTriangles;
}

int32 FStaticMeshLODResources::GetNumVertices() const
{
	return VertexBuffers.StaticMeshVertexBuffer.GetNumVertices();
}

int32 FStaticMeshLODResources::GetNumTexCoords() const
{
	return VertexBuffers.StaticMeshVertexBuffer.GetNumTexCoords();
}

void FStaticMeshVertexFactories::InitVertexFactory(
	const FStaticMeshVertexBuffers& VertexBuffers,
	FLocalVertexFactory& InOutVertexFactory,
	uint32 LODIndex,
	const UStaticMesh* InParentMesh,
	bool bInOverrideColorVertexBuffer
	)
{
	check(InParentMesh != nullptr);

	struct InitStaticMeshVertexFactoryParams
	{
		FLocalVertexFactory* VertexFactory;
		const FStaticMeshVertexBuffers* VertexBuffers;
	#if WITH_EDITORONLY_DATA
		const UStaticMesh* StaticMesh;
	#endif
		uint32 LightMapCoordinateIndex;
		uint32 LODIndex;
		uint8 bOverrideColorVertexBuffer : 1;
	#if WITH_EDITORONLY_DATA
		uint8 bIsCoarseProxy : 1;
	#endif
	} Params;

	uint32 LightMapCoordinateIndex = (uint32)InParentMesh->GetLightMapCoordinateIndex();
	LightMapCoordinateIndex = LightMapCoordinateIndex < VertexBuffers.StaticMeshVertexBuffer.GetNumTexCoords() ? LightMapCoordinateIndex : VertexBuffers.StaticMeshVertexBuffer.GetNumTexCoords() - 1;

	Params.VertexFactory				= &InOutVertexFactory;
	Params.VertexBuffers				= &VertexBuffers;
	Params.bOverrideColorVertexBuffer	= bInOverrideColorVertexBuffer;
	Params.LightMapCoordinateIndex		= LightMapCoordinateIndex;
	Params.LODIndex						= LODIndex;
#if WITH_EDITORONLY_DATA
	Params.StaticMesh					= InParentMesh;
	Params.bIsCoarseProxy				= InParentMesh->IsNaniteEnabled() && InParentMesh->GetNaniteSettings().FallbackPercentTriangles < 1.0f;
#endif

	// Initialize the static mesh's vertex factory.
	ENQUEUE_RENDER_COMMAND(InitStaticMeshVertexFactory)(
		[Params](FRHICommandListImmediate& RHICmdList)
		{
			FLocalVertexFactory::FDataType Data;

			Params.VertexBuffers->PositionVertexBuffer.BindPositionVertexBuffer(Params.VertexFactory, Data);
			Params.VertexBuffers->StaticMeshVertexBuffer.BindTangentVertexBuffer(Params.VertexFactory, Data);
			Params.VertexBuffers->StaticMeshVertexBuffer.BindPackedTexCoordVertexBuffer(Params.VertexFactory, Data);
			Params.VertexBuffers->StaticMeshVertexBuffer.BindLightMapVertexBuffer(Params.VertexFactory, Data, Params.LightMapCoordinateIndex);

			// bOverrideColorVertexBuffer means we intend to override the color later.  We must construct the vertexfactory such that it believes a proper stride (not 0) is set for
			// the color stream so that the real stream works later.
			if(Params.bOverrideColorVertexBuffer)
			{ 
				FColorVertexBuffer::BindDefaultColorVertexBuffer(Params.VertexFactory, Data, FColorVertexBuffer::NullBindStride::FColorSizeForComponentOverride);
			}
			//otherwise just bind the incoming buffer directly.
			else
			{
				Params.VertexBuffers->ColorVertexBuffer.BindColorVertexBuffer(Params.VertexFactory, Data);
			}

			Data.LODLightmapDataIndex	= Params.LODIndex;
		#if WITH_EDITORONLY_DATA
			Data.bIsCoarseProxy			= Params.bIsCoarseProxy;
			Data.StaticMesh				= Params.StaticMesh;
		#endif
			Params.VertexFactory->SetData(RHICmdList, Data);
			Params.VertexFactory->InitResource(RHICmdList);
		});
}

void FStaticMeshVertexFactories::InitResources(const FStaticMeshVertexBuffers& VertexBuffers, uint32 LODIndex, const UStaticMesh* Parent)
{
	InitVertexFactory(VertexBuffers, VertexFactory, LODIndex, Parent, false);
	BeginInitResource(&VertexFactory);

	InitVertexFactory(VertexBuffers, VertexFactoryOverrideColorVertexBuffer, LODIndex, Parent, true);
	BeginInitResource(&VertexFactoryOverrideColorVertexBuffer);
}

void FStaticMeshVertexFactories::ReleaseResources()
{
	// Release the vertex factories.
	BeginReleaseResource(&VertexFactory);
	BeginReleaseResource(&VertexFactoryOverrideColorVertexBuffer);

	if (SplineVertexFactory)
	{
		BeginReleaseResource(SplineVertexFactory);		
	}
	if (SplineVertexFactoryOverrideColorVertexBuffer)
	{
		BeginReleaseResource(SplineVertexFactoryOverrideColorVertexBuffer);		
	}
}

FStaticMeshVertexFactories::~FStaticMeshVertexFactories()
{
	delete SplineVertexFactory;
	delete SplineVertexFactoryOverrideColorVertexBuffer;
}

FStaticMeshSectionAreaWeightedTriangleSampler::FStaticMeshSectionAreaWeightedTriangleSampler()
	: Owner(nullptr)
	, SectionIdx(INDEX_NONE)
{
}

void FStaticMeshSectionAreaWeightedTriangleSampler::Init(FStaticMeshLODResources* InOwner, int32 InSectionIdx)
{
	Owner = InOwner;
	SectionIdx = InSectionIdx;
	Initialize();
}

float FStaticMeshSectionAreaWeightedTriangleSampler::GetWeights(TArray<float>& OutWeights)
{
	//If these hit, you're trying to get weights on a sampler that's not been initialized.
	check(Owner);
	check(SectionIdx != INDEX_NONE);
	check(Owner->Sections.IsValidIndex(SectionIdx));
	FIndexArrayView Indicies = Owner->IndexBuffer.GetArrayView();
	FStaticMeshSection& Section = Owner->Sections[SectionIdx];

	int32 First = Section.FirstIndex;
	int32 Last = First + Section.NumTriangles * 3;
	float Total = 0.0f;
	OutWeights.Empty(Indicies.Num() / 3);
	for (int32 i = First; i < Last; i+=3)
	{
		FVector3f V0 = Owner->VertexBuffers.PositionVertexBuffer.VertexPosition(Indicies[i]);
		FVector3f V1 = Owner->VertexBuffers.PositionVertexBuffer.VertexPosition(Indicies[i + 1]);
		FVector3f V2 = Owner->VertexBuffers.PositionVertexBuffer.VertexPosition(Indicies[i + 2]);

		float Area = ((V1 - V0) ^ (V2 - V0)).Size() * 0.5f;
		OutWeights.Add(Area);
		Total += Area;
	}
	return Total;
}

FStaticMeshAreaWeightedSectionSampler::FStaticMeshAreaWeightedSectionSampler()
	: Owner(nullptr)
{
}

void FStaticMeshAreaWeightedSectionSampler::Init(const FStaticMeshLODResources* InOwner)
{
	Owner = InOwner;
	Initialize();
}

float FStaticMeshAreaWeightedSectionSampler::GetWeights(TArray<float>& OutWeights)
{
	float Total = 0.0f;

	if (Owner)
	{
		//If this hits, you're trying to get weights on a sampler that's not been initialized.
		OutWeights.Empty(Owner->Sections.Num());
		for (int32 i = 0; i < Owner->Sections.Num(); ++i)
		{
			float T = Owner->AreaWeightedSectionSamplers[i].GetTotalWeight();
			OutWeights.Add(T);
			Total += T;
		}

		// Release the reference to the LODresource to avoid blocking stream out operations.
		Owner.SafeRelease();
	}

	return Total;
}

static inline void InitOrUpdateResource(FRHICommandListBase& RHICmdList, FRenderResource* Resource)
{
	if (!Resource->IsInitialized())
	{
		Resource->InitResource(RHICmdList);
	}
	else
	{
		Resource->UpdateRHI(RHICmdList);
	}
}

void FStaticMeshVertexBuffers::InitComponentVF(FLocalVertexFactory* VertexFactory, int32 LightMapCoordinateIndex, bool bOverrideColorVertexBuffer, FLocalVertexFactory::FDataType& OutData) const
{
	PositionVertexBuffer.BindPositionVertexBuffer(VertexFactory, OutData);
	StaticMeshVertexBuffer.BindTangentVertexBuffer(VertexFactory, OutData);
	StaticMeshVertexBuffer.BindPackedTexCoordVertexBuffer(VertexFactory, OutData);
	StaticMeshVertexBuffer.BindLightMapVertexBuffer(VertexFactory, OutData, LightMapCoordinateIndex);
	if (bOverrideColorVertexBuffer)
	{
		FColorVertexBuffer::BindDefaultColorVertexBuffer(VertexFactory, OutData, FColorVertexBuffer::NullBindStride::FColorSizeForComponentOverride);
	}
	else
	{
		ColorVertexBuffer.BindColorVertexBuffer(VertexFactory, OutData);
	}
}

void FStaticMeshVertexBuffers::InitModelBuffers(TArray<FModelVertex>& Vertices)
{
	if (Vertices.Num())
	{
		PositionVertexBuffer.Init(Vertices.Num());
		StaticMeshVertexBuffer.SetUseFullPrecisionUVs(true);
		StaticMeshVertexBuffer.Init(Vertices.Num(), 2);

		for (int32 i = 0; i < Vertices.Num(); i++)
		{
			const FModelVertex& Vertex = Vertices[i];

			PositionVertexBuffer.VertexPosition(i) = Vertex.Position;
			StaticMeshVertexBuffer.SetVertexTangents(i, Vertex.TangentX, Vertex.GetTangentY(), Vertex.TangentZ);
			StaticMeshVertexBuffer.SetVertexUV(i, 0, Vertex.TexCoord);
			StaticMeshVertexBuffer.SetVertexUV(i, 1, Vertex.ShadowTexCoord);
		}
	}
	else
	{
		PositionVertexBuffer.Init(1);
		StaticMeshVertexBuffer.Init(1, 2);

		PositionVertexBuffer.VertexPosition(0) = FVector3f(0, 0, 0);
		StaticMeshVertexBuffer.SetVertexTangents(0, FVector3f(1, 0, 0), FVector3f(0, 1, 0), FVector3f(0, 0, 1));
		StaticMeshVertexBuffer.SetVertexUV(0, 0, FVector2f(0, 0));
		StaticMeshVertexBuffer.SetVertexUV(0, 1, FVector2f(0, 0));
	}
}

void FStaticMeshVertexBuffers::InitModelVF(FRHICommandListBase* RHICmdList, FRenderCommandPipe* RenderCommandPipe, FLocalVertexFactory* VertexFactory)
{
	auto Lambda = [this, VertexFactory](FRHICommandListBase& RHICmdList)
	{
		check(PositionVertexBuffer.IsInitialized());
		check(StaticMeshVertexBuffer.IsInitialized());

		FLocalVertexFactory::FDataType Data;
		PositionVertexBuffer.BindPositionVertexBuffer(VertexFactory, Data);
		StaticMeshVertexBuffer.BindTangentVertexBuffer(VertexFactory, Data);
		StaticMeshVertexBuffer.BindPackedTexCoordVertexBuffer(VertexFactory, Data);
		StaticMeshVertexBuffer.BindLightMapVertexBuffer(VertexFactory, Data, 1);
		FColorVertexBuffer::BindDefaultColorVertexBuffer(VertexFactory, Data, FColorVertexBuffer::NullBindStride::ZeroForDefaultBufferBind);
		VertexFactory->SetData(RHICmdList, Data);

		InitOrUpdateResource(RHICmdList, VertexFactory);
	};

	if (RHICmdList)
	{
		Lambda(*RHICmdList);
	}
	else
	{
		ENQUEUE_RENDER_COMMAND(StaticMeshVertexBuffersInitModelVF)(RenderCommandPipe, MoveTemp(Lambda));
	}
}

void FStaticMeshVertexBuffers::InitWithDummyData(FRHICommandListBase* RHICmdList, FRenderCommandPipe* RenderCommandPipe, FLocalVertexFactory* VertexFactory, uint32 NumVerticies, uint32 NumTexCoords, uint32 LightMapIndex)
{
	check(NumVerticies);
	check(NumTexCoords < MAX_STATIC_TEXCOORDS && NumTexCoords > 0);
	check(LightMapIndex < NumTexCoords);

	PositionVertexBuffer.Init(NumVerticies);
	StaticMeshVertexBuffer.Init(NumVerticies, NumTexCoords);
	ColorVertexBuffer.Init(NumVerticies);

	auto Lambda = [this, VertexFactory, LightMapIndex](FRHICommandListBase& RHICmdList)
	{
		InitOrUpdateResource(RHICmdList, &PositionVertexBuffer);
		InitOrUpdateResource(RHICmdList, &StaticMeshVertexBuffer);
		InitOrUpdateResource(RHICmdList, &ColorVertexBuffer);

		FLocalVertexFactory::FDataType Data;
		PositionVertexBuffer.BindPositionVertexBuffer(VertexFactory, Data);
		StaticMeshVertexBuffer.BindTangentVertexBuffer(VertexFactory, Data);
		StaticMeshVertexBuffer.BindPackedTexCoordVertexBuffer(VertexFactory, Data);
		StaticMeshVertexBuffer.BindLightMapVertexBuffer(VertexFactory, Data, LightMapIndex);
		ColorVertexBuffer.BindColorVertexBuffer(VertexFactory, Data);
		VertexFactory->SetData(RHICmdList, Data);

		InitOrUpdateResource(RHICmdList, VertexFactory);
	};

	if (RHICmdList)
	{
		Lambda(*RHICmdList);
	}
	else
	{
		ENQUEUE_RENDER_COMMAND(StaticMeshVertexBuffersInitWithDummyData)(RenderCommandPipe, MoveTemp(Lambda));
	}
}

void FStaticMeshVertexBuffers::InitFromDynamicVertex(FRHICommandListBase* RHICmdList, FRenderCommandPipe* RenderCommandPipe, FLocalVertexFactory* VertexFactory, TArray<FDynamicMeshVertex>& Vertices, uint32 NumTexCoords, uint32 LightMapIndex)
{
	check(NumTexCoords < MAX_STATIC_TEXCOORDS && NumTexCoords > 0);
	check(LightMapIndex < NumTexCoords);

	if (Vertices.Num())
	{
		PositionVertexBuffer.Init(Vertices.Num());
		StaticMeshVertexBuffer.Init(Vertices.Num(), NumTexCoords);
		ColorVertexBuffer.Init(Vertices.Num());

		for (int32 i = 0; i < Vertices.Num(); i++)
		{
			const FDynamicMeshVertex& Vertex = Vertices[i];

			PositionVertexBuffer.VertexPosition(i) = Vertex.Position;
			StaticMeshVertexBuffer.SetVertexTangents(i, Vertex.TangentX.ToFVector3f(), Vertex.GetTangentY(), Vertex.TangentZ.ToFVector3f());
			for (uint32 j = 0; j < NumTexCoords; j++)
			{
				StaticMeshVertexBuffer.SetVertexUV(i, j, Vertex.TextureCoordinate[j]);
			}
			ColorVertexBuffer.VertexColor(i) = Vertex.Color;
		}
	}
	else
	{
		PositionVertexBuffer.Init(1);
		StaticMeshVertexBuffer.Init(1, 1);
		ColorVertexBuffer.Init(1);

		PositionVertexBuffer.VertexPosition(0) = FVector3f(0, 0, 0);
		StaticMeshVertexBuffer.SetVertexTangents(0, FVector3f(1, 0, 0), FVector3f(0, 1, 0), FVector3f(0, 0, 1));
		StaticMeshVertexBuffer.SetVertexUV(0, 0, FVector2f(0, 0));
		ColorVertexBuffer.VertexColor(0) = FColor(1,1,1,1);
		NumTexCoords = 1;
		LightMapIndex = 0;
	}

	auto Lambda = [this, VertexFactory, LightMapIndex](FRHICommandListBase& RHICmdList)
	{
		InitOrUpdateResource(RHICmdList, &PositionVertexBuffer);
		InitOrUpdateResource(RHICmdList, &StaticMeshVertexBuffer);
		InitOrUpdateResource(RHICmdList, &ColorVertexBuffer);

		FLocalVertexFactory::FDataType Data;
		PositionVertexBuffer.BindPositionVertexBuffer(VertexFactory, Data);
		StaticMeshVertexBuffer.BindTangentVertexBuffer(VertexFactory, Data);
		StaticMeshVertexBuffer.BindPackedTexCoordVertexBuffer(VertexFactory, Data);
		StaticMeshVertexBuffer.BindLightMapVertexBuffer(VertexFactory, Data, LightMapIndex);
		ColorVertexBuffer.BindColorVertexBuffer(VertexFactory, Data);
		VertexFactory->SetData(RHICmdList, Data);

		InitOrUpdateResource(RHICmdList, VertexFactory);
	};

	if (RHICmdList)
	{
		Lambda(*RHICmdList);
	}
	else
	{
		ENQUEUE_RENDER_COMMAND(StaticMeshVertexBuffersInitFromDynamicVertex)(RenderCommandPipe, MoveTemp(Lambda));
	}
};

FStaticMeshLODResources::FStaticMeshLODResources(bool bAddRef)
	: CardRepresentationData(nullptr)
	, MaxDeviation(0.0f)
	, bHasDepthOnlyIndices(false)
	, bHasReversedIndices(false)
	, bHasReversedDepthOnlyIndices(false)
	, bHasColorVertexData(false)
	, bHasWireframeIndices(false)
	, bBuffersInlined(false)
	, bIsOptionalLOD(false)
	, DepthOnlyNumTriangles(0)
	, BuffersSize(0)
#if STATS
	, StaticMeshIndexMemory(0)
#endif
{
	if (bAddRef)
	{
		AddRef();
	}
}

FStaticMeshLODResources::~FStaticMeshLODResources()
{
	check(GetRefCount() == 0);
	delete DistanceFieldData;
	delete CardRepresentationData;
	delete AdditionalIndexBuffers;
	delete RayTracingGeometry;
}

template <bool bIncrement>
void FStaticMeshLODResources::UpdateIndexMemoryStats()
{
#if STATS
	if (bIncrement)
	{
		StaticMeshIndexMemory = IndexBuffer.GetAllocatedSize();
		StaticMeshIndexMemory += DepthOnlyIndexBuffer.GetAllocatedSize();

		if (AdditionalIndexBuffers)
		{
			StaticMeshIndexMemory += AdditionalIndexBuffers->WireframeIndexBuffer.GetAllocatedSize();
			StaticMeshIndexMemory += AdditionalIndexBuffers->ReversedIndexBuffer.GetAllocatedSize();
			StaticMeshIndexMemory += AdditionalIndexBuffers->ReversedDepthOnlyIndexBuffer.GetAllocatedSize();
		}

		INC_DWORD_STAT_BY(STAT_StaticMeshIndexMemory, StaticMeshIndexMemory);
	}
	else
	{
		DEC_DWORD_STAT_BY(STAT_StaticMeshIndexMemory, StaticMeshIndexMemory);
	}
#endif
}

template <bool bIncrement>
void FStaticMeshLODResources::UpdateVertexMemoryStats() const
{
#if STATS
	const uint32 StaticMeshVertexMemory =
		VertexBuffers.StaticMeshVertexBuffer.GetResourceSize() +
		VertexBuffers.PositionVertexBuffer.GetStride() * VertexBuffers.PositionVertexBuffer.GetNumVertices();
	const uint32 ResourceVertexColorMemory = VertexBuffers.ColorVertexBuffer.GetStride() * VertexBuffers.ColorVertexBuffer.GetNumVertices();

	if (bIncrement)
	{
		INC_DWORD_STAT_BY(STAT_StaticMeshVertexMemory, StaticMeshVertexMemory);
		INC_DWORD_STAT_BY(STAT_ResourceVertexColorMemory, ResourceVertexColorMemory);
	}
	else
	{
		DEC_DWORD_STAT_BY(STAT_StaticMeshVertexMemory, StaticMeshVertexMemory);
		DEC_DWORD_STAT_BY(STAT_ResourceVertexColorMemory, ResourceVertexColorMemory);
	}
#endif
}

void FStaticMeshLODResources::InitResources(UStaticMesh* Parent, int32 LODIndex)
{
	const FName OwnerName = UStaticMesh::GetLODPathName(Parent, LODIndex);

	if (bBuffersInlined)
	{
		UpdateIndexMemoryStats<true>();
	}

	BeginInitResource(OwnerName, &IndexBuffer);

	if(bHasWireframeIndices)
	{
		BeginInitResource(OwnerName, &AdditionalIndexBuffers->WireframeIndexBuffer);
	}

	BeginInitResource(OwnerName, &VertexBuffers.StaticMeshVertexBuffer);
	BeginInitResource(OwnerName, &VertexBuffers.PositionVertexBuffer);

	if(bHasColorVertexData)
	{
		BeginInitResource(OwnerName, &VertexBuffers.ColorVertexBuffer);
	}

	if (bHasReversedIndices)
	{
		BeginInitResource(OwnerName, &AdditionalIndexBuffers->ReversedIndexBuffer);
	}

	if (bHasDepthOnlyIndices)
	{
		BeginInitResource(OwnerName, &DepthOnlyIndexBuffer);
	}

	if (bHasReversedDepthOnlyIndices)
	{
		BeginInitResource(OwnerName, &AdditionalIndexBuffers->ReversedDepthOnlyIndexBuffer);
	}

	if (Parent && Parent->bSupportGpuUniformlyDistributedSampling && Parent->bSupportUniformlyDistributedSampling && (AreaWeightedSampler.GetNumEntries() > 0))
	{
		AreaWeightedSectionSamplersBuffer.Init(&AreaWeightedSectionSamplers);
		BeginInitResource(OwnerName, &AreaWeightedSectionSamplersBuffer);
	}

	if (Parent && !Parent->bSupportRayTracing)
	{
		checkf(RayTracingGeometry == nullptr, TEXT("Unexpected RayTracingGeometry on '%s' LOD:%d (ray tracing support is disabled on this UStaticMesh)."), *GetPathNameSafe(Parent), LODIndex);
	}

#if RHI_RAYTRACING
	if (IsRayTracingAllowed() && RayTracingGeometry != nullptr)
	{
		ENQUEUE_RENDER_COMMAND(InitStaticMeshRayTracingGeometry)(
			[this, DebugName = (Parent ? Parent->GetFName() : NAME_None), OwnerName](FRHICommandListImmediate& RHICmdList)
			{
				FRayTracingGeometryInitializer Initializer;
				SetupRayTracingGeometryInitializer(Initializer, DebugName, OwnerName);

				RayTracingGeometry->SetInitializer(Initializer);
			}
		);
	}
#endif // RHI_RAYTRACING

#if STATS
	ENQUEUE_RENDER_COMMAND(UpdateMemoryStats)(
		[this](FRHICommandListImmediate&)
	{
		if (bBuffersInlined)
		{
			UpdateVertexMemoryStats<true>();
		}
	});
#endif

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	ENQUEUE_RENDER_COMMAND(NameRHIResources)(
		[this, DebugName = (Parent ? Parent->GetFName() : NAME_None)](FRHICommandListImmediate& RHICmdList)
	{
		TStringBuilder<512> StringBuilder;
		auto SetDebugName = [&RHICmdList, &StringBuilder, DebugName](FRHIBuffer* RHIBuffer, const TCHAR* Extension)
		{
			if (RHIBuffer)
			{
				StringBuilder.Reset();
				DebugName.ToString(StringBuilder);
				StringBuilder.Append(Extension);
				RHICmdList.BindDebugLabelName(RHIBuffer, StringBuilder.ToString());
			}
		};
		SetDebugName(IndexBuffer.IndexBufferRHI, TEXT("_IB"));
		SetDebugName(VertexBuffers.PositionVertexBuffer.VertexBufferRHI, TEXT("_VB"));
		SetDebugName(VertexBuffers.StaticMeshVertexBuffer.TexCoordVertexBuffer.VertexBufferRHI, TEXT("_TC"));
		SetDebugName(VertexBuffers.StaticMeshVertexBuffer.TangentsVertexBuffer.VertexBufferRHI, TEXT("_TB"));
		SetDebugName(VertexBuffers.ColorVertexBuffer.VertexBufferRHI, TEXT("_CB"));

		SetDebugName(DepthOnlyIndexBuffer.IndexBufferRHI, TEXT("_IB_DepthOnly"));
		if (AdditionalIndexBuffers)
		{
			SetDebugName(AdditionalIndexBuffers->WireframeIndexBuffer.IndexBufferRHI, TEXT("_IB_WireFrame"));
			SetDebugName(AdditionalIndexBuffers->ReversedIndexBuffer.IndexBufferRHI, TEXT("_IB_Reversed"));
			SetDebugName(AdditionalIndexBuffers->ReversedDepthOnlyIndexBuffer.IndexBufferRHI, TEXT("_IB_DepthOnly_Reversed"));
		}
		SetDebugName(AreaWeightedSectionSamplersBuffer.GetBufferRHI(), TEXT("_AreaWeightSectionSamplers"));
	});
#endif // !(UE_BUILD_SHIPPING || UE_BUILD_TEST)

}

#if RHI_RAYTRACING
void FStaticMeshLODResources::SetupRayTracingGeometryInitializer(FRayTracingGeometryInitializer& Initializer, const FDebugName& DebugName, const FName& OwnerName) const
{
	Initializer.DebugName = DebugName;
	Initializer.OwnerName = OwnerName;
	Initializer.IndexBuffer = IndexBuffer.IndexBufferRHI;
	Initializer.TotalPrimitiveCount = 0; // This is calculated below based on static mesh section data
	Initializer.GeometryType = RTGT_Triangles;
	Initializer.bFastBuild = false;

	TArray<FRayTracingGeometrySegment> GeometrySections;
	GeometrySections.Reserve(Sections.Num());
	for (const FStaticMeshSection& Section : Sections)
	{
		FRayTracingGeometrySegment Segment;
		Segment.VertexBuffer = VertexBuffers.PositionVertexBuffer.VertexBufferRHI;
		Segment.VertexBufferElementType = VET_Float3;
		Segment.VertexBufferStride = VertexBuffers.PositionVertexBuffer.GetStride();
		Segment.VertexBufferOffset = 0;
		Segment.MaxVertices = VertexBuffers.PositionVertexBuffer.GetNumVertices();
		Segment.FirstPrimitive = Section.FirstIndex / 3;
		Segment.NumPrimitives = Section.NumTriangles;
		Segment.bEnabled = Section.bVisibleInRayTracing;
		Segment.bForceOpaque = Section.bForceOpaque;
		GeometrySections.Add(Segment);
		Initializer.TotalPrimitiveCount += Section.NumTriangles;
	}
	Initializer.Segments = GeometrySections;
}
#endif // RHI_RAYTRACING

void FStaticMeshLODResources::ReleaseResources()
{
	UpdateVertexMemoryStats<false>();
	UpdateIndexMemoryStats<false>();

	// Release the vertex and index buffers.
	


	BeginReleaseResource(&IndexBuffer);
	
	BeginReleaseResource(&VertexBuffers.StaticMeshVertexBuffer);
	BeginReleaseResource(&VertexBuffers.PositionVertexBuffer);
	BeginReleaseResource(&VertexBuffers.ColorVertexBuffer);
	BeginReleaseResource(&DepthOnlyIndexBuffer);
	BeginReleaseResource(&AreaWeightedSectionSamplersBuffer);

	if (AdditionalIndexBuffers)
	{
		// These may not be initialized at this time, but it is safe to release it anyway.
		// The bInitialized flag will be safely checked in the render thread.
		// This avoids a race condition regarding releasing this resource.
		BeginReleaseResource(&AdditionalIndexBuffers->ReversedIndexBuffer);
		BeginReleaseResource(&AdditionalIndexBuffers->WireframeIndexBuffer);
		BeginReleaseResource(&AdditionalIndexBuffers->ReversedDepthOnlyIndexBuffer);
	}

#if RHI_RAYTRACING
	if (RayTracingGeometry)
	{
		BeginReleaseResource(RayTracingGeometry);
	}
#endif // RHI_RAYTRACING
}

void FStaticMeshLODResources::IncrementMemoryStats()
{
	UpdateIndexMemoryStats<true>();
	UpdateVertexMemoryStats<true>();
}

void FStaticMeshLODResources::DecrementMemoryStats()
{
	UpdateVertexMemoryStats<false>();
	UpdateIndexMemoryStats<false>();
}

void FStaticMeshLODResources::DiscardCPUData()
{
	VertexBuffers.StaticMeshVertexBuffer.CleanUp();
	VertexBuffers.PositionVertexBuffer.CleanUp();
	VertexBuffers.ColorVertexBuffer.CleanUp();
	IndexBuffer.Discard();
	DepthOnlyIndexBuffer.Discard();

	if (AdditionalIndexBuffers)
	{
		AdditionalIndexBuffers->ReversedIndexBuffer.Discard();
		AdditionalIndexBuffers->ReversedDepthOnlyIndexBuffer.Discard();
		AdditionalIndexBuffers->WireframeIndexBuffer.Discard();
	}
	
#if RHI_RAYTRACING
	if (RayTracingGeometry != nullptr)
	{
		RayTracingGeometry->RawData.Discard();
	}
#endif
}

/*------------------------------------------------------------------------------
	FStaticMeshRayTracingProxyLOD
------------------------------------------------------------------------------*/

#if RHI_RAYTRACING
void FStaticMeshRayTracingProxyLOD::SetupRayTracingGeometryInitializer(FRayTracingGeometryInitializer& Initializer, const FDebugName& DebugName, const FName& OwnerName) const
{
	Initializer.DebugName = DebugName;
	Initializer.OwnerName = OwnerName;
	Initializer.IndexBuffer = IndexBuffer->IndexBufferRHI;
	Initializer.TotalPrimitiveCount = 0; // This is calculated below based on static mesh section data
	Initializer.GeometryType = RTGT_Triangles;
	Initializer.bFastBuild = false;

	TArray<FRayTracingGeometrySegment> GeometrySections;
	GeometrySections.Reserve(Sections->Num());
	for (const FStaticMeshSection& Section : *Sections)
	{
		FRayTracingGeometrySegment Segment;
		Segment.VertexBuffer = VertexBuffers->PositionVertexBuffer.VertexBufferRHI;
		Segment.VertexBufferElementType = VET_Float3;
		Segment.VertexBufferStride = VertexBuffers->PositionVertexBuffer.GetStride();
		Segment.VertexBufferOffset = 0;
		Segment.MaxVertices = VertexBuffers->PositionVertexBuffer.GetNumVertices();
		Segment.FirstPrimitive = Section.FirstIndex / 3;
		Segment.NumPrimitives = Section.NumTriangles;
		Segment.bEnabled = Section.bVisibleInRayTracing;
		Segment.bForceOpaque = Section.bForceOpaque;
		GeometrySections.Add(Segment);
		Initializer.TotalPrimitiveCount += Section.NumTriangles;
	}
	Initializer.Segments = GeometrySections;
}
#endif // RHI_RAYTRACING

FStaticMeshRayTracingProxyLOD::~FStaticMeshRayTracingProxyLOD()
{
	if (bOwnsRayTracingGeometry)
	{
		delete RayTracingGeometry;
	}

	if (bOwnsBuffers)
	{
		delete VertexBuffers;
		delete IndexBuffer;
		delete Sections;
	}
}

void FStaticMeshRayTracingProxyLOD::InitResources(UStaticMesh* Parent, int32 LODIndex)
{
	const FName OwnerName = UStaticMesh::GetLODPathName(Parent, LODIndex);

	if (bOwnsBuffers)
	{
		BeginInitResource(OwnerName, IndexBuffer);

		BeginInitResource(OwnerName, &VertexBuffers->StaticMeshVertexBuffer);
		BeginInitResource(OwnerName, &VertexBuffers->PositionVertexBuffer);
		BeginInitResource(OwnerName, &VertexBuffers->ColorVertexBuffer);
	}

	if (Parent && !Parent->bSupportRayTracing)
	{
		checkf(RayTracingGeometry == nullptr, TEXT("Unexpected RayTracingGeometry on '%s' ray tracing proxy LOD:%d (ray tracing support is disabled on this UStaticMesh)."), *GetPathNameSafe(Parent), LODIndex);
	}

#if RHI_RAYTRACING
	if (IsRayTracingAllowed() && RayTracingGeometry != nullptr && bOwnsRayTracingGeometry)
	{
		if (RayTracingGeometry->RawDataHeader.IsValid())
		{
			FRayTracingAccelerationStructureOfflineMetadata OfflineMetadata = RHIGetRayTracingGeometryOfflineMetadata(RayTracingGeometry->RawDataHeader);

			OfflineBVHOffset += OfflineMetadata.SerializedOffset;
			OfflineBVHSize = OfflineMetadata.SerializedSize;
		}

		ENQUEUE_RENDER_COMMAND(InitStaticMeshRayTracingGeometry)(
			[this, DebugName = (Parent ? Parent->GetFName() : NAME_None), OwnerName](FRHICommandListImmediate& RHICmdList)
			{
				FRayTracingGeometryInitializer Initializer;
				SetupRayTracingGeometryInitializer(Initializer, DebugName, OwnerName);

				RayTracingGeometry->SetInitializer(Initializer);
			}
		);
	}
#endif // RHI_RAYTRACING
}

void FStaticMeshRayTracingProxyLOD::ReleaseResources()
{
	if (bOwnsRayTracingGeometry)
	{
		BeginReleaseResource(RayTracingGeometry);
	}

	if (bOwnsBuffers)
	{
		BeginReleaseResource(IndexBuffer);

		BeginReleaseResource(&VertexBuffers->StaticMeshVertexBuffer);
		BeginReleaseResource(&VertexBuffers->PositionVertexBuffer);
		BeginReleaseResource(&VertexBuffers->ColorVertexBuffer);
	}
}

void FStaticMeshRayTracingProxyLOD::DiscardCPUData()
{
	if (bOwnsRayTracingGeometry)
	{
		RayTracingGeometry->RawData.Discard();
	}

	if (bOwnsBuffers)
	{
		VertexBuffers->StaticMeshVertexBuffer.CleanUp();
		VertexBuffers->PositionVertexBuffer.CleanUp();
		VertexBuffers->ColorVertexBuffer.CleanUp();
		IndexBuffer->Discard();
	}
}

void FStaticMeshRayTracingProxyLOD::Serialize(FArchive& Ar, UObject* Owner, int32 Index)
{
	UStaticMesh* OwnerStaticMesh = Cast<UStaticMesh>(Owner);

#if WITH_EDITOR
	// if the ray tracing proxy is using the fallback mesh but it will be stripped from the cooked build
	// the ray tracing proxy needs to adopt the fallback mesh
	const bool bAdoptFallbackMesh = ShouldStripNaniteFallbackMesh(Ar.CookingTarget(), OwnerStaticMesh);
#else
	const bool bAdoptFallbackMesh = false;
#endif

	const bool bOwnsBuffersBackup = bOwnsBuffers;
	const bool bOwnsRayTracingGeometryBackup = bOwnsRayTracingGeometry;

	if (bAdoptFallbackMesh)
	{
		bOwnsBuffers = true;
		bOwnsRayTracingGeometry = true;
	}

	bool bTmpOwnsBuffers = bOwnsBuffers;
	Ar << bTmpOwnsBuffers;
	bOwnsBuffers = bTmpOwnsBuffers;

	if (bOwnsBuffers)
	{
		if (VertexBuffers == nullptr)
		{
			check(Ar.IsLoading());
			VertexBuffers = new FStaticMeshVertexBuffers();
		}

		if (IndexBuffer == nullptr)
		{
			check(Ar.IsLoading());
			IndexBuffer = new FRawStaticIndexBuffer();
		}

		if (Sections == nullptr)
		{
			check(Ar.IsLoading());
			Sections = new FStaticMeshSectionArray();
		}

		Ar << *Sections;
	}

	bool bTmpOwnsRayTracingGeometry = bOwnsRayTracingGeometry;
	Ar << bTmpOwnsRayTracingGeometry;
	bOwnsRayTracingGeometry = bTmpOwnsRayTracingGeometry;

	if (bOwnsRayTracingGeometry && RayTracingGeometry == nullptr)
	{
		check(Ar.IsLoading());
		RayTracingGeometry = new FRayTracingGeometry();
	}

	// TODO: Support DDC streaming in editor
	bool bTmpBuffersInlined = !Ar.IsCooking();
	Ar << bTmpBuffersInlined;
	bBuffersInlined = bTmpBuffersInlined;

	if (bBuffersInlined)
	{
		SerializeBuffers(Ar, OwnerStaticMesh, 0);
		SerializeRayTracingGeometry(Ar, OwnerStaticMesh, 0);
		SerializeMetaData(Ar);
	}
	else
	{
#if WITH_EDITOR
		if (Ar.IsSaving())
		{
			checkf(Ar.IsCooking(), TEXT("Only expected to serialize buffers to FByteBulkData when cooking."));

			TArray<uint8> TmpBuff;

			{
				FMemoryWriter MemWriter(TmpBuff, /*bIsPersistent=*/ true);
				MemWriter.SetSavePackageData(Ar.GetSavePackageData());
				MemWriter.SetByteSwapping(Ar.IsByteSwapping());

				SerializeBuffers(MemWriter, OwnerStaticMesh, 0);
				SerializeRayTracingGeometry(MemWriter, OwnerStaticMesh, 0);
			}

			if(!TmpBuff.IsEmpty())
			{
				StreamableData.Lock(LOCK_READ_WRITE);

				void* Ptr = StreamableData.Realloc(TmpBuff.Num());
				FMemory::Memcpy(Ptr, TmpBuff.GetData(), TmpBuff.Num());

				StreamableData.Unlock();
				StreamableData.SetBulkDataFlags(BULKDATA_Force_NOT_InlinePayload); // TODO: Support inline LOD
			}
		}
#endif
		
		StreamableData.Serialize(Ar, Owner, Index);

		SerializeMetaData(Ar);
	}

	if (bAdoptFallbackMesh)
	{
		// revert back to original state to prevent issues with ownership
		bOwnsBuffers = bOwnsBuffersBackup;
		bOwnsRayTracingGeometry = bOwnsRayTracingGeometryBackup;
	}
}

void FStaticMeshRayTracingProxyLOD::SerializeBuffers(FArchive& Ar, UStaticMesh* OwnerStaticMesh, uint8 InStripFlags)
{
	if (bOwnsBuffers)
	{
		const int64 BeforeOffset = Ar.Tell();

		VertexBuffers->PositionVertexBuffer.Serialize(Ar, false);
		VertexBuffers->StaticMeshVertexBuffer.Serialize(Ar, false);
		VertexBuffers->ColorVertexBuffer.Serialize(Ar, false);

		IndexBuffer->Serialize(Ar, false);

		const int64 AfterOffset = Ar.Tell();

		if (Ar.IsSaving())
		{
			BuffersSize = AfterOffset - BeforeOffset;
		}
	}
}

void FStaticMeshRayTracingProxyLOD::SerializeRayTracingGeometry(FArchive& Ar, UStaticMesh* OwnerStaticMesh, uint8 InStripFlags)
{
	if (bOwnsRayTracingGeometry)
	{
		// logic similar to TArray::BulkSerialize(...)
		// but want to track specific offset of RawData to be able to stream-in specific chunks based on FRayTracingAccelerationStructureOfflineMetadata

		uint32 RawDataNum = RayTracingGeometry->RawData.Num();
		Ar << RawDataNum;

		if (Ar.IsLoading())
		{
			RayTracingGeometry->RawData.Empty(RawDataNum);
			RayTracingGeometry->RawData.AddUninitialized(RawDataNum);
		}

		const int64 BeforeOffset = Ar.Tell();

		Ar.Serialize(RayTracingGeometry->RawData.GetData(), RawDataNum * RayTracingGeometry->RawData.GetTypeSize());

		const int64 AfterOffset = Ar.Tell();

		if (Ar.IsSaving())
		{
			OfflineBVHOffset = BeforeOffset;
			OfflineBVHSize = AfterOffset - BeforeOffset;
		}

		if (Ar.IsLoading() && RayTracing::ShouldForceRuntimeBLAS())
		{
			RayTracingGeometry->RawData.Discard();
		}
	}
}

void FStaticMeshRayTracingProxyLOD::SerializeMetaData(FArchive& Ar)
{
	if (bOwnsBuffers)
	{
		Ar << BuffersSize;

		VertexBuffers->PositionVertexBuffer.SerializeMetaData(Ar);
		VertexBuffers->StaticMeshVertexBuffer.SerializeMetaData(Ar);
		VertexBuffers->ColorVertexBuffer.SerializeMetaData(Ar);

		IndexBuffer->SerializeMetaData(Ar);
	}

	if (bOwnsRayTracingGeometry)
	{
		Ar << OfflineBVHOffset;
		Ar << OfflineBVHSize;

		Ar << RayTracingGeometry->RawDataHeader;

		if (Ar.IsLoading() && RayTracing::ShouldForceRuntimeBLAS())
		{
			RayTracingGeometry->RawDataHeader = FRayTracingGeometryOfflineDataHeader();
		}
	}
}

uint32 FStaticMeshRayTracingProxyLOD::GetRequestSize() const
{
	const uint32 RequestSize = BuffersSize + OfflineBVHSize;
	check(RequestSize < StreamableData.GetBulkDataSize());
	return RequestSize;
}

uint32 FStaticMeshRayTracingProxyLOD::GetRequestSizeBVH() const
{
	const uint32 RequestSize = OfflineBVHSize;
	check(RequestSize < StreamableData.GetBulkDataSize());
	return RequestSize;
}

uint32 FStaticMeshRayTracingProxyLOD::GetRequestSizeBuffers() const
{
	const uint32 RequestSize = BuffersSize;
	check(RequestSize < StreamableData.GetBulkDataSize());
	return RequestSize;
}

bool FStaticMeshRayTracingProxyLOD::AreBuffersStreamedIn() const
{
	return bBuffersStreamedIn;
}

bool FStaticMeshRayTracingProxyLOD::IsBVHStreamedIn() const
{
	return bRayTracingGeometryStreamedIn;
}

void FStaticMeshRayTracingProxyLOD::IssueRequest(FBulkDataBatchRequest& Request, FIoBuffer& RequestBuffer, EAsyncIOPriorityAndFlags Priority, bool bBuffersOnly)
{
	checkf(!StreamableData.IsBulkDataLoaded(), TEXT("Bulk data should not be loaded yet!"));
	checkf(StreamableData.CanLoadFromDisk(), TEXT("Bulk data is not loaded and cannot be loaded from disk!"));
	check(!StreamableData.IsStoredCompressedOnDisk()); // We do not support compressed Bulkdata for this system. Limitation of the streaming request/bulk data

	const bool bRequestBuffers = BuffersSize > 0;
	const bool bRequestOfflineBVH = OfflineBVHSize > 0 && !bBuffersOnly;

	uint32 BatchSize = 0;
	BatchSize += bRequestBuffers ? 1 : 0;
	BatchSize += bRequestOfflineBVH ? 1 : 0;

	// TODO: We're currently using a single batch per request so we can individually cancel and wait on requests.
	// This isn't ideal and should be revisited in the future.
	TRACE_IOSTORE_METADATA_SCOPE_TAG("StaticMeshRayTracingProxyLOD");
	FBulkDataBatchRequest::FScatterGatherBuilder Batch = FBulkDataBatchRequest::ScatterGather(BatchSize);
	if (bRequestBuffers)
	{
		Batch.Read(StreamableData, 0, BuffersSize);
	}
	if (bRequestOfflineBVH)
	{
		Batch.Read(StreamableData, OfflineBVHOffset, OfflineBVHSize);
	}
	Batch.Issue(RequestBuffer, Priority, [](FBulkDataRequest::EStatus)
		{
#if !UE_BUILD_SHIPPING
			// On some platforms the IO is too fast to test cancellation requests timing issues.
			if (FRenderAssetStreamingSettings::ExtraIOLatencyRayTracing > 0)
			{
				FPlatformProcess::Sleep(FRenderAssetStreamingSettings::ExtraIOLatencyRayTracing * .001f); // Slow down the streaming.
			}
#endif
		}, Request);
}

void FStaticMeshRayTracingProxyLOD::InitWithStreamedData(FRHICommandListBase& RHICmdList, FMemoryView StreamedData, bool bBuffersOnly)
{
	FMemoryReaderView Ar(StreamedData, /*bIsPersistent=*/ true);

	SerializeBuffers(Ar, /*OwnerStaticMesh*/nullptr, /*InStripFlags*/0);

	if (bOwnsRayTracingGeometry && OfflineBVHSize > 0 && !bBuffersOnly)
	{
		RayTracingGeometry->RawData.Empty(OfflineBVHSize);
		RayTracingGeometry->RawData.AddUninitialized(OfflineBVHSize);
		Ar.Serialize(RayTracingGeometry->RawData.GetData(), OfflineBVHSize);

		if (Ar.IsLoading() && RayTracing::ShouldForceRuntimeBLAS())
		{
			RayTracingGeometry->RawData.Discard();
		}
	}
	
	check(!Ar.IsError());

	if (bOwnsBuffers)
	{
		FStaticMeshVertexBuffers& VBs = *VertexBuffers;
		FBufferRHIRef TangentsVertexBuffer = VBs.StaticMeshVertexBuffer.CreateTangentsRHIBuffer(RHICmdList);
		FBufferRHIRef TexCoordVertexBuffer = VBs.StaticMeshVertexBuffer.CreateTexCoordRHIBuffer(RHICmdList);
		FBufferRHIRef PositionVertexBuffer = VBs.PositionVertexBuffer.CreateRHIBuffer(RHICmdList);
		FBufferRHIRef ColorVertexBuffer = VBs.ColorVertexBuffer.CreateRHIBuffer(RHICmdList);
		FBufferRHIRef IndexBufferRHI = IndexBuffer->CreateRHIBuffer(RHICmdList);

		FRHIResourceReplaceBatcher Batcher(RHICmdList, 1);

		VBs.StaticMeshVertexBuffer.InitRHIForStreaming(TangentsVertexBuffer, TexCoordVertexBuffer, Batcher);
		VBs.PositionVertexBuffer.InitRHIForStreaming(PositionVertexBuffer, Batcher);
		VBs.ColorVertexBuffer.InitRHIForStreaming(ColorVertexBuffer, Batcher);
		IndexBuffer->InitRHIForStreaming(IndexBufferRHI, Batcher);

		bBuffersStreamedIn = true;
	}

#if RHI_RAYTRACING
	if (bOwnsRayTracingGeometry && !bBuffersOnly)
	{
		{
			FRHIResourceReplaceBatcher Batcher(RHICmdList, 1);
			FRayTracingGeometryInitializer IntermediateInitializer = RayTracingGeometry->GetInitializer();
			IntermediateInitializer.Type = ERayTracingGeometryInitializerType::StreamingSource;

			if (!RayTracingGeometry->RawData.IsEmpty())
			{
				IntermediateInitializer.OfflineData = &RayTracingGeometry->RawData;
			}

			FRayTracingGeometryRHIRef IntermediateRayTracingGeometry = RHICmdList.CreateRayTracingGeometry(IntermediateInitializer);

			RayTracingGeometry->SetRequiresBuild(IntermediateInitializer.OfflineData == nullptr || IntermediateRayTracingGeometry->IsCompressed());

			RayTracingGeometry->InitRHIForStreaming(IntermediateRayTracingGeometry, Batcher);

			// When Batcher goes out of scope it will add commands to copy the BLAS buffers on RHI thread.
			// We need to do it before we build the current geometry (also on RHI thread).
		}

		if (!RayTracingGeometry->GetRequiresBuild())
		{
			// only need to request here if no build will be requested since build path already requests update as necessary
			GRayTracingGeometryManager->RequestUpdateCachedRenderState(RayTracingGeometry->GroupHandle);
		}

		RayTracingGeometry->RequestBuildIfNeeded(RHICmdList, ERTAccelerationStructureBuildPriority::Normal);

		bRayTracingGeometryStreamedIn = true;
	}
#endif // RHI_RAYTRACING
}

void FStaticMeshRayTracingProxyLOD::ReleaseForStreaming(FRHIResourceReplaceBatcher& Batcher)
{
	ReleaseBVHForStreaming(Batcher);
	ReleaseBuffersForStreaming(Batcher);
}

void FStaticMeshRayTracingProxyLOD::ReleaseBVHForStreaming(FRHIResourceReplaceBatcher& Batcher)
{
#if RHI_RAYTRACING
	if (bOwnsRayTracingGeometry && bRayTracingGeometryStreamedIn)
	{
		RayTracingGeometry->ReleaseRHIForStreaming(Batcher);

		bRayTracingGeometryStreamedIn = false;
	}
#endif // RHI_RAYTRACING
}

void FStaticMeshRayTracingProxyLOD::ReleaseBuffersForStreaming(FRHIResourceReplaceBatcher& Batcher)
{
	if (bOwnsBuffers && bBuffersStreamedIn)
	{
		VertexBuffers->PositionVertexBuffer.ReleaseRHIForStreaming(Batcher);
		VertexBuffers->StaticMeshVertexBuffer.ReleaseRHIForStreaming(Batcher);
		VertexBuffers->ColorVertexBuffer.ReleaseRHIForStreaming(Batcher);

		IndexBuffer->ReleaseRHIForStreaming(Batcher);

		bBuffersStreamedIn = false;
	}
}

/*------------------------------------------------------------------------------
	FStaticMeshRayTracingProxy
------------------------------------------------------------------------------*/

FStaticMeshRayTracingProxy::~FStaticMeshRayTracingProxy()
{
	if (!bUsingRenderingLODs)
	{
		delete LODVertexFactories;
	}
}

void FStaticMeshRayTracingProxy::InitResources(UStaticMesh* Owner)
{
	for (int32 LODIndex = 0; LODIndex < LODs.Num(); ++LODIndex)
	{
		LODs[LODIndex].InitResources(Owner, LODIndex);

		if (!bUsingRenderingLODs)
		{
			(*LODVertexFactories)[LODIndex].InitResources(*LODs[LODIndex].VertexBuffers, LODIndex, Owner);
		}
	}
}

void FStaticMeshRayTracingProxy::ReleaseResources()
{
	for (int32 LODIndex = 0; LODIndex < LODs.Num(); ++LODIndex)
	{
		LODs[LODIndex].ReleaseResources();

		if (!bUsingRenderingLODs)
		{
			(*LODVertexFactories)[LODIndex].ReleaseResources();
		}
	}
}

void FStaticMeshRayTracingProxy::Serialize(FArchive& Ar, UObject* Owner, FStaticMeshRenderData* RenderData, bool bCooked)
{
	FStripDataFlags StripFlags(Ar);

	UStaticMesh* OwnerStaticMesh = Cast<UStaticMesh>(Owner);

#if WITH_EDITOR
	// if the ray tracing proxy is using the fallback mesh but it will be stripped from the cooked build
	// the ray tracing proxy needs to adopt the fallback mesh
	const bool bAdoptFallbackMesh = ShouldStripNaniteFallbackMesh(Ar.CookingTarget(), OwnerStaticMesh);
#else
	const bool bAdoptFallbackMesh = false;
#endif

	const bool bUsingRenderingLODsBackup = bUsingRenderingLODs;

	if (bAdoptFallbackMesh)
	{
		bUsingRenderingLODs = false;
	}

	Ar << bUsingRenderingLODs;

	if (!StripFlags.IsAudioVisualDataStripped())
	{
		LODs.Serialize(Ar, Owner);

		if (Ar.IsLoading())
		{
			if (bUsingRenderingLODs)
			{
				FStaticMeshLODResourcesArray& LODResources = RenderData->LODResources;
				check(LODs.Num() == LODResources.Num());

				for (int32 Index = 0; Index < LODs.Num(); ++Index)
				{
					FStaticMeshRayTracingProxyLOD& RayTracingLOD = LODs[Index];

					checkf(RayTracingLOD.bOwnsBuffers == false, TEXT("FStaticMeshRayTracingProxyLOD shouldn't own VB/IB/etc when bUsingRenderingLODs since they're owned by FStaticMeshLODResources"));

					RayTracingLOD.Sections = &LODResources[Index].Sections;
					RayTracingLOD.VertexBuffers = &LODResources[Index].VertexBuffers;
					RayTracingLOD.IndexBuffer = &LODResources[Index].IndexBuffer;

					if (!RayTracingLOD.bOwnsRayTracingGeometry)
					{
						RayTracingLOD.RayTracingGeometry = LODResources[Index].RayTracingGeometry;
					}
				}

				LODVertexFactories = &RenderData->LODVertexFactories;
			}
			else
			{
				LODVertexFactories = new FStaticMeshVertexFactoriesArray;
				LODVertexFactories->Empty(LODs.Num());
				for (int i = 0; i < LODs.Num(); i++)
				{
					new (*LODVertexFactories) FStaticMeshVertexFactories(GMaxRHIFeatureLevel);
				}
			}
		}
	}

#if WITH_EDITOR
	if (bAdoptFallbackMesh)
	{
		// revert back to original state to prevent issues with ownership
		bUsingRenderingLODs = bUsingRenderingLODsBackup;
	}
#endif
}

/*------------------------------------------------------------------------------
	FStaticMeshRenderData
------------------------------------------------------------------------------*/

FStaticMeshRenderData::FStaticMeshRenderData()
	: bLODsShareStaticLighting(false)
	, bHasNaniteFallbackMesh(false)
	, bReadyForStreaming(false)
	, NumInlinedLODs(0)
	, CurrentFirstLODIdx(0)
	, LODBiasModifier(0)
{
	for (int32 LODIndex = 0; LODIndex < MAX_STATIC_MESH_LODS; ++LODIndex)
	{
		ScreenSize[LODIndex] = 0.0f;
	}

	ClearNaniteResources(NaniteResourcesPtr);
}

FStaticMeshRenderData::~FStaticMeshRenderData()
{
	FreeLODResources();

	if (RayTracingProxy != nullptr)
	{
		delete RayTracingProxy;
	}
}

void FStaticMeshRenderData::FreeLODResources()
{
	FStaticMeshLODResources** LODResourcesArray = LODResources.GetData();
	for (int32 LODIndex = 0; LODIndex < LODResources.Num(); ++LODIndex)
	{
		LODResourcesArray[LODIndex]->Release();
		// Prevent the array from calling the destructor to handle correctly the refcount.
		// For compatibility reason, LODResourcesArray is using ptr directly instead of TRefCountPtr.
		LODResourcesArray[LODIndex] = nullptr;
	}
	LODResources.Empty();
}

int32 FStaticMeshRenderData::GetNumNonStreamingLODs() const
{
	int32 Tmp = 0;
	for (int32 Idx = LODResources.Num() - 1; Idx >= 0; --Idx)
	{
		if (!LODResources[Idx].bBuffersInlined)
		{
			break;
		}
		++Tmp;
	}

	if (Tmp == 0 && LODResources.Num())
	{
		return 1;
	}
	else
	{
		return Tmp;
	}
}

int32 FStaticMeshRenderData::GetNumNonOptionalLODs() const
{
	int32 NumNonOptionalLODs = 0;
	for (int32 Idx = LODResources.Num() - 1; Idx >= 0; --Idx)
	{
		const FStaticMeshLODResources& Resource = LODResources[Idx];
		if (!Resource.bIsOptionalLOD)
		{
			++NumNonOptionalLODs;
		}
		else // Stop at the first optional LOD
		{
			break;
		}
	}

	if (NumNonOptionalLODs == 0 && LODResources.Num())
	{
		return 1;
	}
	else
	{
		return NumNonOptionalLODs;
	}
}

void FStaticMeshRenderData::SerializeInlineDataRepresentations(FArchive& Ar, UStaticMesh* Owner)
{
	// Defined class flags for possible stripping
	const uint8 CardRepresentationDataStripFlag = 2;

	// Actual flags used during serialization
	uint8 ClassDataStripFlags = 0;

#if WITH_EDITOR
	const bool bWantToStripCardRepresentationData = Ar.IsCooking() && (!Ar.CookingTarget()->SupportsFeature(ETargetPlatformFeatures::DeferredRendering) || !Ar.CookingTarget()->SupportsFeature(ETargetPlatformFeatures::LumenGI));
	ClassDataStripFlags |= (bWantToStripCardRepresentationData ? CardRepresentationDataStripFlag : 0);
#endif

	FStripDataFlags StripFlags(Ar, ClassDataStripFlags);
	if (!StripFlags.IsAudioVisualDataStripped() && !StripFlags.IsClassDataStripped(CardRepresentationDataStripFlag))
	{
		if (Ar.IsSaving())
		{
			GCardRepresentationAsyncQueue->BlockUntilBuildComplete(Owner, false);
		}

		for (int32 ResourceIndex = 0; ResourceIndex < LODResources.Num(); ResourceIndex++)
		{
			FStaticMeshLODResources& LOD = LODResources[ResourceIndex];
				
			bool bValid = (LOD.CardRepresentationData != nullptr);

			Ar << bValid;

			if (bValid)
			{
				if (LOD.CardRepresentationData == nullptr)
				{
					check(Ar.IsLoading());
					LOD.CardRepresentationData = new FCardRepresentationData();
				}

#if !UE_BUILD_SHIPPING
				if (Ar.IsSaving() && LOD.CardRepresentationData->ContainsNaN())
				{
					UE_LOGF(LogStaticMesh, Display, "FStaticMeshRenderData::SerializeInlineDataRepresentations found NaN when saving CardRepresentationData of '%ls'. LOD:%d", *GetPathNameSafe(Owner), ResourceIndex);
				}
#endif // UE_BUILD_SHIPPING

				Ar << *(LOD.CardRepresentationData);
				
#if !UE_BUILD_SHIPPING
				if (Ar.IsLoading() && LOD.CardRepresentationData->ContainsNaN())
				{
					UE_LOGF(LogStaticMesh, Display, "FStaticMeshRenderData::SerializeInlineDataRepresentations found NaN when loading CardRepresentationData of '%ls'. LOD:%d", *GetPathNameSafe(Owner), ResourceIndex);
				}
#endif // UE_BUILD_SHIPPING
			}
		}
	}
}

void FStaticMeshRenderData::Serialize(FArchive& Ar, UStaticMesh* Owner, bool bCooked)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FStaticMeshRenderData::Serialize);

	DECLARE_SCOPE_CYCLE_COUNTER( TEXT("FStaticMeshRenderData::Serialize"), STAT_StaticMeshRenderData_Serialize, STATGROUP_LoadTime );

	// Note: this is all derived data, native versioning is not needed, but be sure to bump STATICMESH_DERIVEDDATA_VER when modifying!
#if WITH_EDITOR
	const bool bHasEditorData = !Owner->GetOutermost()->bIsCookedForEditor;
	if (Ar.IsSaving() && bHasEditorData)
	{
		ResolveSectionInfo(Owner);
	}
#endif
#if WITH_EDITORONLY_DATA
	if (!bCooked)
	{
		Ar << MaterialIndexToImportIndex;
		Ar << EstimatedNaniteTotalCompressedSize;
		Ar << EstimatedNaniteStreamingCompressedSize;
	}

#endif // #if WITH_EDITORONLY_DATA

#if PLATFORM_DESKTOP
	if (bCooked)
	{
		int32 MinMobileLODIdx = 0;
		bool bShouldSerialize = CVarStaticMeshKeepMobileMinLODSettingOnDesktop.GetValueOnAnyThread() != 0;
#if WITH_EDITOR
		if (Ar.IsSaving())
		{
			if (Ar.CookingTarget()->GetPlatformInfo().PlatformGroupName == TEXT("Desktop")
				&& CVarStripMinLodDataDuringCooking.GetValueOnAnyThread() != 0
				&& CVarStaticMeshKeepMobileMinLODSettingOnDesktop.GetValueOnAnyThread() != 0)
			{
				// Serialize 0 value when per quality level properties are used
				if (!Owner->IsMinLodQualityLevelEnable())
				{
					MinMobileLODIdx = Owner->GetMinLOD().GetValueForPlatform(TEXT("Mobile")) - FStaticMeshLODResources::GetPlatformMinLODIdx(Ar.CookingTarget(), Owner);
					// Will be cast to uint8 when applying LOD bias. Also, make sure it's not < 0,
					// which can happen if the desktop min LOD is higher than the mobile setting
					MinMobileLODIdx = FMath::Clamp(MinMobileLODIdx, 0, 255);
				}
			}
			else
			{
				bShouldSerialize = false;
			}
		}
#endif

		if (bShouldSerialize)
		{
			Ar << MinMobileLODIdx;

			if (Ar.IsLoading() && GMaxRHIFeatureLevel == ERHIFeatureLevel::ES3_1)
			{
				LODBiasModifier = MinMobileLODIdx;
			}
		}
	}
#endif // PLATFORM_DESKTOP

	if (Ar.IsLoading() && !LODResources.IsEmpty())
	{
		FreeLODResources();
	}
	LODResources.Serialize(Ar, Owner);

#if WITH_EDITOR
	if (Ar.IsSaving())
	{
		NumInlinedLODs = GetNumNonStreamingLODs();
	}
#endif
	Ar << NumInlinedLODs;

#if WITH_EDITOR
	if (bCooked && Ar.IsLoading())
	{
		CurrentFirstLODIdx = Owner->GetMinLODIdx();
	}
	else
#endif
	{
		CurrentFirstLODIdx = LODResources.Num() - NumInlinedLODs;
	}

	if (Ar.IsLoading())
	{
		LODVertexFactories.Empty(LODResources.Num());
		for (int i = 0; i < LODResources.Num(); i++)
		{
			new (LODVertexFactories) FStaticMeshVertexFactories(GMaxRHIFeatureLevel);
		}
	}

	check(NaniteResourcesPtr.IsValid());
	NaniteResourcesPtr->Serialize(Ar, Owner, bCooked);

	{
		bool bHasRayTracingProxy = (RayTracingProxy != nullptr);

#if WITH_EDITOR
		if (Ar.IsCooking() && !Ar.CookingTarget()->UsesRayTracing())
		{
			bHasRayTracingProxy = false; // strip ray tracing proxy
		}
#endif

		Ar << bHasRayTracingProxy;

		if (bHasRayTracingProxy)
		{
			if (RayTracingProxy == nullptr)
			{
				checkf(Ar.IsLoading(), TEXT("bHasRayTracingProxy unexpectedly changed even though we are not loading data."));
				RayTracingProxy = new FStaticMeshRayTracingProxy();
			}

			RayTracingProxy->Serialize(Ar, Owner, this, bCooked);
		}
		else if (RayTracingProxy != nullptr)
		{
			delete RayTracingProxy;
			RayTracingProxy = nullptr;
		}
	}

	// Inline the distance field derived data for cooked builds
	if (bCooked)
	{
		SerializeInlineDataRepresentations(Ar, Owner);

		// Defined class flags for possible stripping
		const uint8 DistanceFieldDataStripFlag = 1;

		// Actual flags used during serialization
		uint8 ClassDataStripFlags = 0;

#if WITH_EDITOR
		const bool bWantToStripDistanceFieldData = Ar.IsCooking() 
			&& (!Ar.CookingTarget()->SupportsFeature(ETargetPlatformFeatures::DistanceFieldAO) || !Ar.CookingTarget()->UsesDistanceFields());

		ClassDataStripFlags |= (bWantToStripDistanceFieldData ? DistanceFieldDataStripFlag : 0);
#endif

		FStripDataFlags StripFlags(Ar, ClassDataStripFlags);
		if (!StripFlags.IsAudioVisualDataStripped() && !StripFlags.IsClassDataStripped(DistanceFieldDataStripFlag))
		{
			if (Ar.IsSaving())
			{
				GDistanceFieldAsyncQueue->BlockUntilBuildComplete(Owner, false);
			}

			for (int32 ResourceIndex = 0; ResourceIndex < LODResources.Num(); ResourceIndex++)
			{
				FStaticMeshLODResources& LOD = LODResources[ResourceIndex];
				
				bool bValid = (LOD.DistanceFieldData != nullptr);

				Ar << bValid;

				if (bValid)
				{
#if WITH_EDITOR
					if (Ar.IsCooking() && Ar.IsSaving())
					{
						check(LOD.DistanceFieldData != nullptr);

						float Divider = Ar.CookingTarget()->GetDownSampleMeshDistanceFieldDivider();

						if (Divider > 1)
						{
							//@todo - strip mips
							LOD.DistanceFieldData->Serialize(Ar, Owner);
						}
						else
						{
							LOD.DistanceFieldData->Serialize(Ar, Owner);
						}
					}
					else
#endif
					{
						if (LOD.DistanceFieldData == nullptr)
						{
							LOD.DistanceFieldData = new FDistanceFieldVolumeData();
							LOD.DistanceFieldData->AssetName = Owner->GetFName();
						}

#if !UE_BUILD_SHIPPING
						if (LOD.DistanceFieldData->LocalSpaceMeshBounds.ContainsNaN())
						{
							UE_LOGF(LogStaticMesh, Warning, "FStaticMeshRenderData::Serialize found NaN in DistanceFieldData of '%ls'. LOD:%d", *GetPathNameSafe(Owner), ResourceIndex);
						}
#endif // UE_BUILD_SHIPPING
						LOD.DistanceFieldData->Serialize(Ar, Owner);
					}
				}
			}
		}
	}

	Ar << Bounds;
	
#if !UE_BUILD_SHIPPING
	if (Bounds.ContainsNaN())
	{
		UE_LOGF(LogStaticMesh, Warning, "FStaticMeshRenderData::Serialize found NaN in Bounds of '%ls'", *GetPathNameSafe(Owner));
	}
#endif // !UE_BUILD_SHIPPING

#if WITH_EDITORONLY_DATA
	if (bHasNaniteFallbackMesh && Ar.IsCooking() && Ar.IsSaving())
	{
		bHasNaniteFallbackMesh = (Owner->GetNaniteSettings().GenerateFallback == ENaniteGenerateFallback::Enabled || !Ar.CookingTarget()->ShouldStripNaniteFallbackMeshes());
	}
#endif // WITH_EDITORONLY_DATA

	// pack multiple flags into a single uint8
	static const uint8 LODsShareStaticLightingFlag = 1;
	static const uint8 HasNaniteFallbackMeshFlag = 2;

	uint8 bRenderDataFlags = 0;
	bRenderDataFlags |= bLODsShareStaticLighting ? LODsShareStaticLightingFlag : 0;
	bRenderDataFlags |= bHasNaniteFallbackMesh ? HasNaniteFallbackMeshFlag : 0;

	Ar << bRenderDataFlags;

	bLODsShareStaticLighting = (bRenderDataFlags & LODsShareStaticLightingFlag) != 0;
	bHasNaniteFallbackMesh = (bRenderDataFlags & HasNaniteFallbackMeshFlag) != 0;

#if WITH_EDITORONLY_DATA
	// store a copy of NaniteSettings.GenerateFallback in FStaticMeshRenderData
	// to be used in HasNaniteFallbackMesh(...) since we don't have access to the owner's UStaticMesh at that point
	bForceGenerateNaniteFallbackMesh = Owner->GetNaniteSettings().GenerateFallback == ENaniteGenerateFallback::Enabled;
#endif

	if (Ar.IsLoading() && Ar.CustomVer(FRenderingObjectVersion::GUID) < FRenderingObjectVersion::TextureStreamingMeshUVChannelData)
	{
		float DummyFactor;
		for (int32 TexCoordIndex = 0; TexCoordIndex < MAX_STATIC_TEXCOORDS; ++TexCoordIndex)
		{
			Ar << DummyFactor; // StreamingTextureFactors[TexCoordIndex];
		}
		Ar << DummyFactor; // MaxStreamingTextureFactor;
	}

	if (bCooked)
	{
		for (int32 LODIndex = 0; LODIndex < MAX_STATIC_MESH_LODS; ++LODIndex)
		{
			Ar << ScreenSize[LODIndex];
		}
	}

	if (Ar.IsLoading() )
	{
		bool bStripDistanceFieldDataDuringLoad = (CVarStripDistanceFieldDataDuringLoad.GetValueOnAnyThread() == 1);
		if( bStripDistanceFieldDataDuringLoad )
		{
			for (int32 ResourceIndex = 0; ResourceIndex < LODResources.Num(); ResourceIndex++)
			{
				FStaticMeshLODResources& LOD = LODResources[ResourceIndex];
				if( LOD.DistanceFieldData != nullptr )
				{
					delete LOD.DistanceFieldData;
					LOD.DistanceFieldData = nullptr;
				}
			}
		}
	}

#if WITH_EDITORONLY_DATA
	// when cooking for a cooked cooker, we need to save extra data it may later need from its own cooked assets
	if (Ar.IsCooking())
	{
		FStripDataFlags StripFlags(Ar);
		if (!StripFlags.IsDataNeededForCookingStripped())
		{
			// if we need to keep data needed for cooking, just save the collision data
			UStaticMesh* OwnerStaticMesh = Cast<UStaticMesh>(Owner);
			check(OwnerStaticMesh);

			FTriMeshCollisionData CollisionData;
			if (OwnerStaticMesh->ContainsPhysicsTriMeshData(true))
			{
				OwnerStaticMesh->GetPhysicsTriMeshData(&CollisionData, true);
			}

			Ar << CollisionData;
		}
	}
#endif
	if (bCooked && Ar.IsLoading())
	{
		FStripDataFlags StripFlags(Ar);
#if WITH_EDITORONLY_DATA	// the below lines can only happen for a cooked cooker, which has editor data
		if (!StripFlags.IsDataNeededForCookingStripped())
		{
			CollisionDataForCookedCooker = MakeUnique<FTriMeshCollisionData>();
			Ar << *CollisionDataForCookedCooker;
		}
#endif
	}
}

void FStaticMeshRenderData::InitResources(ERHIFeatureLevel::Type InFeatureLevel, UStaticMesh* Owner)
{
	// TODO: This check is disabled because in some cases we end up calling InitResources(...) on already initialized RenderData
	// due to an issue with PostEditUndo + async builds (UE-365996)
	//check(!bIsInitialized);

#if WITH_EDITOR
	// Init the section info only for uncooked editor.
	// Cooked packages don't need this and don't want any LOD screen size changes that it applies.
	if (!Owner->GetPackage()->HasAnyPackageFlags(PKG_FilterEditorOnly))
	{
		ResolveSectionInfo(Owner);
	}
#endif // #if WITH_EDITOR

	checkf(FApp::CanEverRender() || !FPlatformProperties::RequiresCookedData(), TEXT("RenderData should not initialize resources in headless cooked runs"));

	for (int32 LODIndex = 0; LODIndex < LODResources.Num(); ++LODIndex)
	{
		// Skip LODs that have their render data stripped
		if (LODResources[LODIndex].VertexBuffers.StaticMeshVertexBuffer.GetNumVertices() > 0)
		{
			LODResources[LODIndex].InitResources(Owner, LODIndex);
			LODVertexFactories[LODIndex].InitResources(LODResources[LODIndex].VertexBuffers, LODIndex, Owner);
		}
	}

	check(NaniteResourcesPtr.IsValid());
	NaniteResourcesPtr->InitResources(Owner);

	if (RayTracingProxy != nullptr)
	{
		RayTracingProxy->InitResources(Owner);
	}

#if RHI_RAYTRACING
	if (IsRayTracingAllowed() && RayTracingProxy != nullptr) // TODO: Could move most of this to FStaticMeshLODResources::InitResources
	{
		checkf(Owner->bSupportRayTracing, TEXT("Unexpected RayTracingProxy on '%s' (support ray tracing is disabled on this UStaticMesh)."), *GetPathNameSafe(Owner));

		ENQUEUE_RENDER_COMMAND(InitStaticMeshRayTracingGeometry)(
			[this, Owner](FRHICommandListImmediate& RHICmdList)
			{
				FStaticMeshRayTracingProxyLODArray& RayTracingLODs = RayTracingProxy->LODs;

				check(!RayTracingProxy->bUsingRenderingLODs || RayTracingLODs.Num() == LODResources.Num());

				if (RayTracingGeometryGroupHandle.IsNull()) // workaround for FStaticMeshRenderData::InitResources(...) being called on already initialized RenderData due to bug with PostEditUndo + async build (UE-365996)
				{
					RayTracingGeometryGroupHandle = GRayTracingGeometryManager->RegisterRayTracingGeometryGroup(RayTracingLODs.Num(), RayTracingProxy->bUsingRenderingLODs ? CurrentFirstLODIdx : 0);
				}

				for (int32 LODIndex = 0; LODIndex < RayTracingLODs.Num(); ++LODIndex)
				{
					FStaticMeshRayTracingProxyLOD& RayTracingLOD = RayTracingLODs[LODIndex];

					// Skip LODs that have their render data stripped
					if (RayTracingLOD.VertexBuffers->StaticMeshVertexBuffer.GetNumVertices() > 0)
					{
						RayTracingLOD.RayTracingGeometry->GroupHandle = RayTracingGeometryGroupHandle;

						const bool bHasStreamableData = RayTracingLOD.StreamableData.GetBulkDataSize() > 0 && RayTracingLOD.GetRequestSize() > 0;

						if (bHasStreamableData)
						{
							checkf(RayTracingLOD.bOwnsRayTracingGeometry, TEXT("Unexpected StreamableData on '%s' (StreamableData is only expected when ray tracing proxy owns the geometry)."), *GetPathNameSafe(Owner));
							checkf(!RayTracingLOD.bBuffersInlined, TEXT("RayTracing LOD with streamable data shouldn't have buffers inlined"));
						}

						if (bHasStreamableData || LODIndex < CurrentFirstLODIdx)
						{
							PRAGMA_DISABLE_DEPRECATION_WARNINGS
							// TODO: Shouldn't directly modify FRayTracingGeometry initializer
							RayTracingLOD.RayTracingGeometry->Initializer.Type = ERayTracingGeometryInitializerType::StreamingDestination;
							PRAGMA_ENABLE_DEPRECATION_WARNINGS
						}

						RayTracingLOD.RayTracingGeometry->LODIndex = LODIndex;
						RayTracingLOD.RayTracingGeometry->InitResource(RHICmdList);

						if (bHasStreamableData)
						{
							((FRayTracingGeometryManager*)GRayTracingGeometryManager)->SetRayTracingGeometryStreamingData(
								RayTracingLOD.RayTracingGeometry,
								RayTracingLOD);
						}
					}
				}
			}
		);
	}
#endif // RHI_RAYTRACING

	ENQUEUE_RENDER_COMMAND(CmdSetStaticMeshReadyForStreaming)(
		[this, Owner](FRHICommandListImmediate&)
	{
		bReadyForStreaming = true;
	});
	bIsInitialized = true;
}

void FStaticMeshRenderData::ReleaseResources()
{
	const bool bWasInitialized = bIsInitialized;

	bIsInitialized = false;

	if (RayTracingProxy != nullptr)
	{
		RayTracingProxy->ReleaseResources();
	}

	for (int32 LODIndex = 0; LODIndex < LODResources.Num(); ++LODIndex)
	{
		if (LODResources[LODIndex].VertexBuffers.StaticMeshVertexBuffer.GetNumVertices() > 0)
		{
			LODResources[LODIndex].ReleaseResources();
			LODVertexFactories[LODIndex].ReleaseResources();
		}
	}

#if RHI_RAYTRACING
	if (bWasInitialized && IsRayTracingAllowed())
	{
		ENQUEUE_RENDER_COMMAND(CmdReleaseRayTracingGeometryGroup)(
			[this](FRHICommandListImmediate&)
			{
				if (!RayTracingGeometryGroupHandle.IsNull())
				{
					GRayTracingGeometryManager->ReleaseRayTracingGeometryGroup(RayTracingGeometryGroupHandle);
					RayTracingGeometryGroupHandle = {};
				}
			});
	}
#endif

	check(NaniteResourcesPtr.IsValid());
	NaniteResourcesPtr->ReleaseResources();
}

void FStaticMeshRenderData::AllocateLODResources(int32 NumLODs)
{
	check(LODResources.Num() == 0);
	LODResources.Reserve(NumLODs);
	LODVertexFactories.Reserve(NumLODs);
	while (LODResources.Num() < NumLODs)
	{
		LODResources.Add(new FStaticMeshLODResources);
		new (LODVertexFactories) FStaticMeshVertexFactories(GMaxRHIFeatureLevel);
	}
}

void FStaticMeshRenderData::InitializeRayTracingRepresentationFromRenderingLODs()
{
	const bool bProxyOwnsRayTracingGeometry = IsRayTracingEnableOnDemandSupported();

	const int32 NumLODs = LODResources.Num();

	if (!bProxyOwnsRayTracingGeometry)
	{
		for (int32 LODIndex = 0; LODIndex < NumLODs; ++LODIndex)
		{
			FStaticMeshLODResources& LODModel = LODResources[LODIndex];

			checkf(LODModel.RayTracingGeometry == nullptr, TEXT("LODModel.RayTracingGeometry expected to be null. Was the static mesh ray tracing representation already initialized?"));
			LODModel.RayTracingGeometry = new FRayTracingGeometry();
		}
	}

	checkf(RayTracingProxy == nullptr, TEXT("RayTracingProxy expected to be null. Was the static mesh ray tracing representation already initialized?"));
	RayTracingProxy = new FStaticMeshRayTracingProxy();
	RayTracingProxy->bUsingRenderingLODs = true;

	TIndirectArray<FStaticMeshRayTracingProxyLOD>& RayTracingLODs = RayTracingProxy->LODs;
	check(RayTracingLODs.IsEmpty());
	RayTracingLODs.Reserve(NumLODs);

	for (int32 LODIndex = 0; LODIndex < NumLODs; ++LODIndex)
	{
		FStaticMeshLODResources& LODModel = LODResources[LODIndex];

		FStaticMeshRayTracingProxyLOD* RayTracingLOD = new FStaticMeshRayTracingProxyLOD;

		RayTracingLOD->bOwnsRayTracingGeometry = bProxyOwnsRayTracingGeometry;
		RayTracingLOD->RayTracingGeometry = RayTracingLOD->bOwnsRayTracingGeometry ? new FRayTracingGeometry() : LODModel.RayTracingGeometry;

		RayTracingLOD->Sections = &LODModel.Sections;
		RayTracingLOD->VertexBuffers = &LODModel.VertexBuffers;
		RayTracingLOD->IndexBuffer = &LODModel.IndexBuffer;
		RayTracingLOD->bOwnsBuffers = false;

		RayTracingLODs.Add(RayTracingLOD);
	}

	RayTracingProxy->LODVertexFactories = &LODVertexFactories;
}

int32 FStaticMeshRenderData::GetFirstValidLODIdx(int32 MinIdx) const
{
	const int32 LODCount = LODResources.Num();
	int32 LODIndex = INDEX_NONE;
	if (LODCount > 0)
	{
		LODIndex = FMath::Clamp<int32>(MinIdx, 0, LODCount - 1);

		while (LODIndex < LODCount && (LODResources[LODIndex].GetNumVertices() == 0 || LODResources[LODIndex].BuffersSize == 0))
		{
			++LODIndex;
		}

		if (LODIndex >= LODCount)
		{
			LODIndex = INDEX_NONE;
		}
	}

	return LODIndex;
}


void UStaticMesh::RequestUpdateCachedRenderState() const
{
#if RHI_RAYTRACING
	if (IsRayTracingAllowed() && bSupportRayTracing)
	{
		if (GetRenderData()->RayTracingProxy->bUsingRenderingLODs)
		{
			((FRayTracingGeometryManager*)GRayTracingGeometryManager)->SetRayTracingGeometryGroupCurrentFirstLODIndex(FRHICommandListImmediate::Get(), GetRenderData()->RayTracingGeometryGroupHandle, GetRenderData()->CurrentFirstLODIdx);
		}

		// TODO: this could be skipped in some cases
		// - compressed offline BLAS (since build request handle it)
		// - FStaticMeshLODResources doesn't own the ray tracing geometry
		GRayTracingGeometryManager->RequestUpdateCachedRenderState(GetRenderData()->RayTracingGeometryGroupHandle);
	}
#endif

	// TODO: Need to mark all DynamicRayTracingGeometries used in FStaticMeshSceneProxy referencing this StaticMesh as either invalid or request a recreation (UE-139474)
}

bool FStaticMeshRenderData::HasValidNaniteData() const
{
	return NaniteResourcesPtr->PageStreamingStates.Num() > 0;
}

bool FStaticMeshRenderData::HasNaniteFallbackMesh(EShaderPlatform ShaderPlatform) const
{
	checkf(HasValidNaniteData(), TEXT("Should only call HasNaniteFallbackMesh(...) if HasValidNaniteData() returns true."));

#if WITH_EDITORONLY_DATA
	// fallback meshes are always available in editor
	// so here we emulate the value for the requested platform
	if (AreNaniteFallbackMeshesEnabledForPlatform(ShaderPlatform))
	{
		return true;
	}

	return bForceGenerateNaniteFallbackMesh;
#else
	return bHasNaniteFallbackMesh;
#endif
}

#if WITH_EDITOR
/**
 * Calculates the view distance that a mesh should be displayed at.
 * @param MaxDeviation - The maximum surface-deviation between the reduced geometry and the original. This value should be acquired from Simplygon
 * @returns The calculated view distance	 
 */
static float CalculateViewDistance(float MaxDeviation, float AllowedPixelError)
{
	// We want to solve for the depth in world space given the screen space distance between two pixels
	//
	// Assumptions:
	//   1. There is no scaling in the view matrix.
	//   2. The horizontal FOV is 90 degrees.
	//   3. The backbuffer is 1920x1080.
	//
	// If we project two points at (X,Y,Z) and (X',Y,Z) from view space, we get their screen
	// space positions: (X/Z, Y'/Z) and (X'/Z, Y'/Z) where Y' = Y * AspectRatio.
	//
	// The distance in screen space is then sqrt( (X'-X)^2/Z^2 + (Y'-Y')^2/Z^2 )
	// or (X'-X)/Z. This is in clip space, so PixelDist = 1280 * 0.5 * (X'-X)/Z.
	//
	// Solving for Z: ViewDist = (X'-X * 640) / PixelDist

	const float ViewDistance = (MaxDeviation * 960.0f) / FMath::Max(AllowedPixelError, UStaticMesh::MinimumAutoLODPixelError);
	return ViewDistance;
}

void FStaticMeshRenderData::ResolveSectionInfo(UStaticMesh* Owner)
{
	int32 LODIndex = 0;
	int32 MaxLODs = LODResources.Num();
	check(MaxLODs <= MAX_STATIC_MESH_LODS);
	for (; LODIndex < MaxLODs; ++LODIndex)
	{
		FStaticMeshLODResources& LOD = LODResources[LODIndex];
		for (int32 SectionIndex = 0; SectionIndex < LOD.Sections.Num(); ++SectionIndex)
		{
			FMeshSectionInfo Info = Owner->GetSectionInfoMap().Get(LODIndex,SectionIndex);
			FStaticMeshSection& Section = LOD.Sections[SectionIndex];
			Section.MaterialIndex = Info.MaterialIndex;
			Section.bEnableCollision = Info.bEnableCollision;
			Section.bCastShadow = Info.bCastShadow;
			Section.bVisibleInRayTracing = Info.bVisibleInRayTracing;
			Section.bAffectDistanceFieldLighting = Info.bAffectDistanceFieldLighting;
			Section.bForceOpaque = Info.bForceOpaque;
		}

		if (Owner->GetAutoComputeLODScreenSize())
		{
			using namespace UE::StaticMesh::Private;

			if (LODIndex == 0)
			{
				ScreenSize[LODIndex].Default = Constants::LOD0ScreenSize;
			}
			else if(LOD.MaxDeviation <= 0.0f)
			{
				ScreenSize[LODIndex].Default = FMath::Pow(Constants::AutoComputeLODPowerBase, LODIndex);
			}
			else
			{
				const float PixelError = Owner->IsSourceModelValid(LODIndex) ? Owner->GetSourceModel(LODIndex).ReductionSettings.PixelError : UStaticMesh::MinimumAutoLODPixelError;
				const float ViewDistance = CalculateViewDistance(LOD.MaxDeviation, PixelError);

				// Generate a projection matrix.
				// ComputeBoundsScreenSize only uses (0, 0) and (1, 1) of this matrix.
				const float HalfFOV = UE_PI * 0.25f;
				const float ScreenWidth = 1920.0f;
				const float ScreenHeight = 1080.0f;
				const FPerspectiveMatrix ProjMatrix(HalfFOV, ScreenWidth, ScreenHeight, 1.0f);

				// Note we offset ViewDistance by SphereRadius here because the MaxDeviation is known to be somewhere in the bounds of the mesh. 
				// It won't necessarily be at the origin. Before adding this factor for very high poly meshes it would calculate a very small deviation 
				// for LOD1 which translates to a very small ViewDistance and a large (larger than 1) ScreenSize. This meant you could clip the camera 
				// into the mesh but unless you were near its origin it wouldn't switch to LOD0. Adding SphereRadius to ViewDistance makes it so that 
				// the distance is to the bounds which corrects the problem.
				ScreenSize[LODIndex].Default = ComputeBoundsScreenSize(FVector::ZeroVector, Bounds.SphereRadius, FVector(0.0f, 0.0f, ViewDistance + Bounds.SphereRadius), ProjMatrix);
			}
			
			//We must enforce screen size coherence between LOD when we autocompute the LOD screensize
			//This case can happen if we mix auto generate LOD with custom LOD
			if (LODIndex > 0 && ScreenSize[LODIndex].Default > ScreenSize[LODIndex - 1].Default)
			{
				ScreenSize[LODIndex].Default = ScreenSize[LODIndex - 1].Default / 2.0f;
			}
		}
		else if (Owner->IsSourceModelValid(LODIndex))
		{
			ScreenSize[LODIndex] = Owner->GetSourceModel(LODIndex).ScreenSize;
		}
		else
		{
			// No valid source model and we're not auto-generating. Auto-generate in this case
			// because we have nothing else to go on.
			check(LODIndex > 0);
			ScreenSize[LODIndex].Default = UStaticMesh::ComputeLODScreenSize(LODIndex, ScreenSize[LODIndex - 1].Default);
		}
	}
	for (; LODIndex < MAX_STATIC_MESH_LODS; ++LODIndex)
	{
		ScreenSize[LODIndex].Default = 0.0f;
	}
}

void FStaticMeshRenderData::SyncUVChannelData(const TArray<FStaticMaterial>& ObjectData)
{
	TUniquePtr< TArray<FMeshUVChannelInfo> > UpdateData = MakeUnique< TArray<FMeshUVChannelInfo> >();
	UpdateData->Empty(ObjectData.Num());

	for (const FStaticMaterial& StaticMaterial : ObjectData)
	{
		UpdateData->Add(StaticMaterial.UVChannelData);
	}

	// SyncUVChannelData can be called from any thread during async mesh compilation. 
	// There is currently multiple race conditions in ENQUEUE_RENDER_COMMAND making it unsafe to be called from
	// any other thread than rendering or game because of the render thread suspension mecanism.
	// We sidestep the issue here by avoiding a call to ENQUEUE_RENDER_COMMAND if the resource has not been initialized and is still unknown
	// to the render thread.
	if (bIsInitialized)
	{
		ENQUEUE_RENDER_COMMAND(SyncUVChannelData)([this, UpdateData = MoveTemp(UpdateData)](FRHICommandListImmediate& RHICmdList)
		{
			Swap(UVChannelDataPerMaterial, *UpdateData.Get());
		});
	}
	else
	{
		Swap(UVChannelDataPerMaterial, *UpdateData.Get());
	}
}

/*------------------------------------------------------------------------------
	FStaticMeshLODSettings
------------------------------------------------------------------------------*/
void FStaticMeshLODSettings::Initialize(const ITargetPlatformSettings* TargetPlatformSettings)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FStaticMeshLODSettings::Initialize);

	check(!Groups.Num());
	// Ensure there is a default LOD group.
	Groups.FindOrAdd(NAME_None);

	// Read individual entries from a config file.
	const TCHAR* IniSection = TEXT("StaticMeshLODSettings");
	const FConfigSection* Section = TargetPlatformSettings->GetConfigSystem()->GetSection(IniSection, false, GEngineIni);
	if (Section)
	{
		for (TMultiMap<FName,FConfigValue>::TConstIterator It(*Section); It; ++It)
		{
			FName GroupName = It.Key();
			FStaticMeshLODGroup& Group = Groups.FindOrAdd(GroupName);
			ReadEntry(Group, It.Value().GetValue());
		};
	}

	Groups.KeySort(FNameLexicalLess());
	GroupName2Index.Empty(Groups.Num());
	{
		int32 GroupIdx = 0;
		TMap<FName, FStaticMeshLODGroup>::TConstIterator It(Groups);
		for (; It; ++It, ++GroupIdx)
		{
			GroupName2Index.Add(It.Key(), GroupIdx);
		}
	}

	// Do some per-group initialization.
	for (TMap<FName,FStaticMeshLODGroup>::TIterator It(Groups); It; ++It)
	{
		FStaticMeshLODGroup& Group = It.Value();
		EStaticMeshReductionTerimationCriterion LODTerminationCriterion = Group.DefaultSettings[0].TerminationCriterion;
		float PercentTrianglesPerLODRatio = Group.DefaultSettings[1].PercentTriangles;
		float PercentVerticesPerLODRatio = Group.DefaultSettings[1].PercentVertices;
		for (int32 LODIndex = 1; LODIndex < MAX_STATIC_MESH_LODS; ++LODIndex)
		{
			//Set the termination criteria
			Group.DefaultSettings[LODIndex].TerminationCriterion = LODTerminationCriterion;
			float PercentTriangles = Group.DefaultSettings[LODIndex - 1].PercentTriangles;
			float PercentVertices = Group.DefaultSettings[LODIndex - 1].PercentVertices;

			//Clamp Absolute value so every LOD is equal or less the previous LOD
			uint32 MaxNumOfTriangles = FMath::Clamp<uint32>(
				Group.DefaultSettings[LODIndex].MaxNumOfTriangles
				, 2
				, Group.DefaultSettings[LODIndex - 1].MaxNumOfTriangles);
			uint32 MaxNumOfVerts = FMath::Clamp<uint32>(
				Group.DefaultSettings[LODIndex].MaxNumOfVerts
				, 4
				, Group.DefaultSettings[LODIndex - 1].MaxNumOfVerts);

			//Copy the previous LOD
			Group.DefaultSettings[LODIndex] = Group.DefaultSettings[LODIndex - 1];

			//Reduce the data from the previous LOD using the ratios
			Group.DefaultSettings[LODIndex].PercentTriangles = PercentTriangles * PercentTrianglesPerLODRatio;
			Group.DefaultSettings[LODIndex].PercentVertices = PercentVertices * PercentVerticesPerLODRatio;

			//Put back the absolute criterion after the LOD copy
			Group.DefaultSettings[LODIndex].MaxNumOfTriangles = MaxNumOfTriangles;
			Group.DefaultSettings[LODIndex].MaxNumOfVerts = MaxNumOfVerts;
		}
	}
}
void FStaticMeshLODSettings::Initialize(const ITargetPlatform* TargetPlatform)
{
	Initialize(TargetPlatform->GetTargetPlatformSettings());
}

void FStaticMeshLODSettings::ReadEntry(FStaticMeshLODGroup& Group, FString Entry)
{
	FMeshReductionSettings& Settings = Group.DefaultSettings[0];
	FMeshReductionSettings& Bias = Group.SettingsBias;
	int32 Importance = EMeshFeatureImportance::Normal;

	// Trim whitespace at the beginning.
	Entry.TrimStartInline();

	FParse::Value(*Entry, TEXT("Name="), Group.DisplayName, TEXT("StaticMeshLODSettings"));

	// Remove brackets.
	Entry = Entry.Replace( TEXT("("), TEXT("") );
	Entry = Entry.Replace( TEXT(")"), TEXT("") );
		
	if (FParse::Value(*Entry, TEXT("NumLODs="), Group.DefaultNumLODs))
	{
		Group.DefaultNumLODs = FMath::Clamp<int32>(Group.DefaultNumLODs, 1, MAX_STATIC_MESH_LODS);
	}

	if (FParse::Value(*Entry, TEXT("MaxNumStreamedLODs="), Group.DefaultMaxNumStreamedLODs))
	{
		Group.DefaultMaxNumStreamedLODs = FMath::Max(Group.DefaultMaxNumStreamedLODs, 0);
	}

	if (FParse::Value(*Entry, TEXT("MaxNumOptionalLODs="), Group.DefaultMaxNumOptionalLODs))
	{
		Group.DefaultMaxNumOptionalLODs = FMath::Max(Group.DefaultMaxNumOptionalLODs, 0);
	}
	
	int32 LocalSupportLODStreaming = 0;
	if (FParse::Value(*Entry, TEXT("bSupportLODStreaming="), LocalSupportLODStreaming))
	{
		Group.bSupportLODStreaming = !!LocalSupportLODStreaming;
	}

	if (FParse::Value(*Entry, TEXT("LightMapResolution="), Group.DefaultLightMapResolution))
	{
		Group.DefaultLightMapResolution = FMath::Max<int32>(Group.DefaultLightMapResolution, 0);
		Group.DefaultLightMapResolution = (Group.DefaultLightMapResolution + 3) & (~3);
	}

	FString TerminationCriterion = StaticEnum<EStaticMeshReductionTerimationCriterion>()->GetValueAsString(EStaticMeshReductionTerimationCriterion::Triangles);
	if (FParse::Value(*Entry, TEXT("TerminationCriterion="), TerminationCriterion))
	{
		Group.DefaultSettings[0].TerminationCriterion = static_cast<EStaticMeshReductionTerimationCriterion>(StaticEnum<EStaticMeshReductionTerimationCriterion>()->GetValueByNameString(TerminationCriterion));
	}
	float BasePercentTriangles = 100.0f;
	if (FParse::Value(*Entry, TEXT("BasePercentTriangles="), BasePercentTriangles))
	{
		BasePercentTriangles = FMath::Clamp<float>(BasePercentTriangles, 0.0f, 100.0f);
		Group.DefaultSettings[0].PercentTriangles = BasePercentTriangles * 0.01f;
	}

	float BasePercentVertices = 100.0f;
	if (FParse::Value(*Entry, TEXT("BasePercentVertices="), BasePercentVertices))
	{
		BasePercentVertices = FMath::Clamp<float>(BasePercentVertices, 0.0f, 100.0f);
		Group.DefaultSettings[0].PercentVertices = BasePercentVertices * 0.01f;
	}

	float LODPercentTriangles = 100.0f;
	if (FParse::Value(*Entry, TEXT("LODPercentTriangles="), LODPercentTriangles))
	{
		LODPercentTriangles = FMath::Clamp<float>(LODPercentTriangles, 0.0f, 100.0f);
		Group.DefaultSettings[1].PercentTriangles = LODPercentTriangles * 0.01f;
	}

	float LODPercentVertices = 100.0f;
	if (FParse::Value(*Entry, TEXT("LODPercentVertices="), LODPercentVertices))
	{
		LODPercentVertices = FMath::Clamp<float>(LODPercentVertices, 0.0f, 100.0f);
		Group.DefaultSettings[1].PercentVertices = LODPercentVertices * 0.01f;
	}

	for (int32 LodIndex = 0; LodIndex < Group.DefaultNumLODs; ++LodIndex)
	{
		uint32 LODMaxNumOfTriangles = MAX_uint32;
		FString KeySearch = FString::Printf(TEXT("LOD%dMaxNumOfTriangles="), LodIndex);
		if (FParse::Value(*Entry, *KeySearch, LODMaxNumOfTriangles))
		{
			Group.DefaultSettings[LodIndex].MaxNumOfTriangles = LODMaxNumOfTriangles;
		}

		uint32 LODMaxNumOfVerts = MAX_uint32;
		KeySearch = FString::Printf(TEXT("LOD%dMaxNumOfVertices="), LodIndex);
		if (FParse::Value(*Entry, *KeySearch, LODMaxNumOfVerts))
		{
			Group.DefaultSettings[LodIndex].MaxNumOfVerts = LODMaxNumOfVerts;
		}
	}

	if (FParse::Value(*Entry, TEXT("MaxDeviation="), Settings.MaxDeviation))
	{
		Settings.MaxDeviation = FMath::Clamp<float>(Settings.MaxDeviation, 0.0f, 1000.0f);
	}

	if (FParse::Value(*Entry, TEXT("PixelError="), Settings.PixelError))
	{
		Settings.PixelError = FMath::Clamp<float>(Settings.PixelError, 1.0f, 1000.0f);
	}

	if (FParse::Value(*Entry, TEXT("WeldingThreshold="), Settings.WeldingThreshold))
	{
		Settings.WeldingThreshold = FMath::Clamp<float>(Settings.WeldingThreshold, 0.0f, 10.0f);
	}

	if (FParse::Value(*Entry, TEXT("HardAngleThreshold="), Settings.HardAngleThreshold))
	{
		Settings.HardAngleThreshold = FMath::Clamp<float>(Settings.HardAngleThreshold, 0.0f, 180.0f);
	}

	if (FParse::Value(*Entry, TEXT("SilhouetteImportance="), Importance))
	{
		Settings.SilhouetteImportance = (EMeshFeatureImportance::Type)FMath::Clamp<int32>(Importance, 0, EMeshFeatureImportance::Highest);
	}

	if (FParse::Value(*Entry, TEXT("TextureImportance="), Importance))
	{
		Settings.TextureImportance = (EMeshFeatureImportance::Type)FMath::Clamp<int32>(Importance, 0, EMeshFeatureImportance::Highest);
	}

	if (FParse::Value(*Entry, TEXT("ShadingImportance="), Importance))
	{
		Settings.ShadingImportance = (EMeshFeatureImportance::Type)FMath::Clamp<int32>(Importance, 0, EMeshFeatureImportance::Highest);
	}

	float BasePercentTrianglesMult = 100.0f;
	if (FParse::Value(*Entry, TEXT("BasePercentTrianglesMult="), BasePercentTrianglesMult))
	{
		BasePercentTrianglesMult = FMath::Clamp<float>(BasePercentTrianglesMult, 0.0f, 100.0f);
		Group.BasePercentTrianglesMult = BasePercentTrianglesMult * 0.01f;
	}

	float BasePercentVerticesMult = 100.0f;
	if (FParse::Value(*Entry, TEXT("BasePercentVerticesMult="), BasePercentVerticesMult))
	{
		BasePercentVerticesMult = FMath::Clamp<float>(BasePercentVerticesMult, 0.0f, 100.0f);
		Group.BasePercentVerticesMult = BasePercentVerticesMult * 0.01f;
	}

	float LODPercentTrianglesMult = 100.0f;
	if (FParse::Value(*Entry, TEXT("LODPercentTrianglesMult="), LODPercentTrianglesMult))
	{
		LODPercentTrianglesMult = FMath::Clamp<float>(LODPercentTrianglesMult, 0.0f, 100.0f);
		Bias.PercentTriangles = LODPercentTrianglesMult * 0.01f;
	}

	float LODPercentVerticesMult = 100.0f;
	if (FParse::Value(*Entry, TEXT("LODPercentVerticesMult="), LODPercentVerticesMult))
	{
		LODPercentVerticesMult = FMath::Clamp<float>(LODPercentVerticesMult, 0.0f, 100.0f);
		Bias.PercentVertices = LODPercentVerticesMult * 0.01f;
	}

	if (FParse::Value(*Entry, TEXT("MaxDeviationBias="), Bias.MaxDeviation))
	{
		Bias.MaxDeviation = FMath::Clamp<float>(Bias.MaxDeviation, -1000.0f, 1000.0f);
	}

	if (FParse::Value(*Entry, TEXT("PixelErrorBias="), Bias.PixelError))
	{
		Bias.PixelError = FMath::Clamp<float>(Bias.PixelError, 1.0f, 1000.0f);
	}

	if (FParse::Value(*Entry, TEXT("WeldingThresholdBias="), Bias.WeldingThreshold))
	{
		Bias.WeldingThreshold = FMath::Clamp<float>(Bias.WeldingThreshold, -10.0f, 10.0f);
	}

	if (FParse::Value(*Entry, TEXT("HardAngleThresholdBias="), Bias.HardAngleThreshold))
	{
		Bias.HardAngleThreshold = FMath::Clamp<float>(Bias.HardAngleThreshold, -180.0f, 180.0f);
	}

	if (FParse::Value(*Entry, TEXT("SilhouetteImportanceBias="), Importance))
	{
		Bias.SilhouetteImportance = (EMeshFeatureImportance::Type)FMath::Clamp<int32>(Importance, -EMeshFeatureImportance::Highest, EMeshFeatureImportance::Highest);
	}

	if (FParse::Value(*Entry, TEXT("TextureImportanceBias="), Importance))
	{
		Bias.TextureImportance = (EMeshFeatureImportance::Type)FMath::Clamp<int32>(Importance, -EMeshFeatureImportance::Highest, EMeshFeatureImportance::Highest);
	}

	if (FParse::Value(*Entry, TEXT("ShadingImportanceBias="), Importance))
	{
		Bias.ShadingImportance = (EMeshFeatureImportance::Type)FMath::Clamp<int32>(Importance, -EMeshFeatureImportance::Highest, EMeshFeatureImportance::Highest);
	}
}

float UStaticMesh::ComputeLODScreenSize(int32 LODIndex, float PreviousLODScreenSize /*= -1.0f*/)
{
	using namespace UE::StaticMesh::Private;

	if (PreviousLODScreenSize < 0.0f)
	{
		// Provides a good upper bound
		PreviousLODScreenSize = Constants::LOD0ScreenSize;
	}

	if (LODIndex <= 0)
	{
		return Constants::LOD0ScreenSize;
	}

	const float AutoDisplayFactor = FMath::Pow(Constants::AutoComputeLODPowerBase, LODIndex);
	// Make sure this fits in with the previous LOD
	return FMath::Clamp(AutoDisplayFactor, 0.0f, PreviousLODScreenSize - Constants::Tolerance);
}

void FStaticMeshLODSettings::GetLODGroupNames(TArray<FName>& OutNames) const
{
	for (TMap<FName,FStaticMeshLODGroup>::TConstIterator It(Groups); It; ++It)
	{
		OutNames.Add(It.Key());
	}
}

void FStaticMeshLODSettings::GetLODGroupDisplayNames(TArray<FText>& OutDisplayNames) const
{
	for (TMap<FName,FStaticMeshLODGroup>::TConstIterator It(Groups); It; ++It)
	{
		OutDisplayNames.Add( It.Value().DisplayName );
	}
}

FMeshReductionSettings FStaticMeshLODGroup::GetSettings(const FMeshReductionSettings& InSettings, int32 LODIndex) const
{
	check(LODIndex >= 0 && LODIndex < MAX_STATIC_MESH_LODS);

	FMeshReductionSettings FinalSettings = InSettings;

	// PercentTriangles is actually a multiplier.
	float PercentTrianglesMult = (LODIndex == 0) ? BasePercentTrianglesMult : SettingsBias.PercentTriangles;
	FinalSettings.PercentTriangles = FMath::Clamp(InSettings.PercentTriangles * PercentTrianglesMult, 0.0f, 1.0f);

	float PercentVerticesMult = (LODIndex == 0) ? BasePercentVerticesMult : SettingsBias.PercentVertices;
	FinalSettings.PercentVertices = FMath::Clamp(InSettings.PercentVertices * PercentVerticesMult, 0.0f, 1.0f);

	// Bias the remaining settings.
	FinalSettings.MaxDeviation = FMath::Max(InSettings.MaxDeviation + SettingsBias.MaxDeviation, 0.0f);
	FinalSettings.PixelError = FMath::Max(InSettings.PixelError + SettingsBias.PixelError, 1.0f);
	FinalSettings.WeldingThreshold = FMath::Max(InSettings.WeldingThreshold + SettingsBias.WeldingThreshold, 0.0f);
	FinalSettings.HardAngleThreshold = FMath::Clamp(InSettings.HardAngleThreshold + SettingsBias.HardAngleThreshold, 0.0f, 180.0f);
	FinalSettings.SilhouetteImportance = (EMeshFeatureImportance::Type)FMath::Clamp<int32>(InSettings.SilhouetteImportance + SettingsBias.SilhouetteImportance, EMeshFeatureImportance::Off, EMeshFeatureImportance::Highest);
	FinalSettings.TextureImportance = (EMeshFeatureImportance::Type)FMath::Clamp<int32>(InSettings.TextureImportance + SettingsBias.TextureImportance, EMeshFeatureImportance::Off, EMeshFeatureImportance::Highest);
	FinalSettings.ShadingImportance = (EMeshFeatureImportance::Type)FMath::Clamp<int32>(InSettings.ShadingImportance + SettingsBias.ShadingImportance, EMeshFeatureImportance::Off, EMeshFeatureImportance::Highest);
	return FinalSettings;
}

void UStaticMesh::GetLODGroups(TArray<FName>& OutLODGroups)
{
	ITargetPlatform* RunningPlatform = GetTargetPlatformManagerRef().GetRunningTargetPlatform();
	check(RunningPlatform);
	RunningPlatform->GetStaticMeshLODSettings().GetLODGroupNames(OutLODGroups);
}

void UStaticMesh::GetLODGroupsDisplayNames(TArray<FText>& OutLODGroupsDisplayNames)
{
	ITargetPlatform* RunningPlatform = GetTargetPlatformManagerRef().GetRunningTargetPlatform();
	check(RunningPlatform);
	RunningPlatform->GetStaticMeshLODSettings().GetLODGroupDisplayNames(OutLODGroupsDisplayNames);
}

bool UStaticMesh::IsReductionActive(int32 LodIndex) const
{
	//Invalid LOD are not reduced
	if (!IsSourceModelValid(LodIndex))
	{
		return false;
	}

	bool bReductionActive = false;
	if (IMeshReduction* ReductionModule = FModuleManager::Get().LoadModuleChecked<IMeshReductionManagerModule>("MeshReductionInterface").GetStaticMeshReductionInterface())
	{
		FMeshReductionSettings ReductionSettings = GetReductionSettings(LodIndex);
		const FStaticMeshSourceModel& SrcModel = GetSourceModel(LodIndex);
		uint32 LODTriNumber = SrcModel.CacheMeshDescriptionTrianglesCount;
		uint32 LODVertexNumber = SrcModel.CacheMeshDescriptionVerticesCount;
		bReductionActive = ReductionModule->IsReductionActive(ReductionSettings, LODVertexNumber, LODTriNumber);
	}
	return bReductionActive;
}

FMeshReductionSettings UStaticMesh::GetReductionSettings(int32 LODIndex) const
{
	check(IsSourceModelValid(LODIndex));
	//Retrieve the reduction settings, make sure we use the LODGroup if the Group is valid
	ITargetPlatformManagerModule& TargetPlatformManager = GetTargetPlatformManagerRef();
	ITargetPlatform* RunningPlatform = TargetPlatformManager.GetRunningTargetPlatform();
	check(RunningPlatform);
	const FStaticMeshLODSettings& LODSettings = RunningPlatform->GetStaticMeshLODSettings();
	const FStaticMeshLODGroup& SMLODGroup = LODSettings.GetLODGroup(GetLODGroup());
	const FStaticMeshSourceModel& SrcModel = GetSourceModel(LODIndex);
	return SMLODGroup.GetSettings(SrcModel.ReductionSettings, LODIndex);
}

bool UStaticMesh::GetEnableLODStreaming(const ITargetPlatform* TargetPlatform) const
{
	if (NeverStream)
	{
		return false;
	}

	static auto* VarMeshStreaming = IConsoleManager::Get().FindConsoleVariable(TEXT("r.MeshStreaming"));
	const bool bMeshStreamingDisabled = VarMeshStreaming && VarMeshStreaming->GetInt() == 0;

	if (bMeshStreamingDisabled)
	{
		return false;

	}

	check(TargetPlatform);
	// Check whether the target platforms supports LOD streaming. 
	// Even if it does, disable streaming if it has editor only data since most tools don't support mesh streaming.
	if (!TargetPlatform->SupportsFeature(ETargetPlatformFeatures::MeshLODStreaming) || TargetPlatform->HasEditorOnlyData())
	{
		return false;
	}

	switch (LODStreaming)
	{
		case EStaticMeshLODStreaming::Enabled:
			return true;
		case EStaticMeshLODStreaming::Disabled:
			return false;
		case EStaticMeshLODStreaming::Default:
		default:
		{
			const FStaticMeshLODGroup& LODGroupSettings = TargetPlatform->GetStaticMeshLODSettings().GetLODGroup(GetLODGroup());
			return LODGroupSettings.IsLODStreamingSupported();
		}
	}
}

void UStaticMesh::PostDuplicate(bool bDuplicateForPIE)
{
	Super::PostDuplicate(bDuplicateForPIE);

	if (!bDuplicateForPIE)
	{
		SetLightingGuid();
	}
}

static void SerializeReductionSettingsForDDC(FArchive& Ar, FMeshReductionSettings& ReductionSettings)
{
	// Note: this serializer is only used to build the mesh DDC key, no versioning is required
	Ar << ReductionSettings.TerminationCriterion;
	Ar << ReductionSettings.PercentTriangles;
	Ar << ReductionSettings.MaxNumOfTriangles;
	Ar << ReductionSettings.PercentVertices;
	Ar << ReductionSettings.MaxNumOfVerts;
	Ar << ReductionSettings.MaxDeviation;
	Ar << ReductionSettings.PixelError;
	Ar << ReductionSettings.WeldingThreshold;
	Ar << ReductionSettings.HardAngleThreshold;
	Ar << ReductionSettings.SilhouetteImportance;
	Ar << ReductionSettings.TextureImportance;
	Ar << ReductionSettings.ShadingImportance;
	Ar << ReductionSettings.BaseLODModel;
	FArchive_Serialize_BitfieldBool(Ar, ReductionSettings.bRecalculateNormals);
}

static void SerializeBuildSettingsForDDC(FArchive& Ar, FMeshBuildSettings& BuildSettings)
{
	// Note: this serializer is only used to build the mesh DDC key, no versioning is required
	FArchive_Serialize_BitfieldBool(Ar, BuildSettings.bRecomputeNormals);
	FArchive_Serialize_BitfieldBool(Ar, BuildSettings.bRecomputeTangents);
	FArchive_Serialize_BitfieldBool(Ar, BuildSettings.bUseMikkTSpace);
	FArchive_Serialize_BitfieldBool(Ar, BuildSettings.bComputeWeightedNormals);
	FArchive_Serialize_BitfieldBool(Ar, BuildSettings.bRemoveDegenerates);
	FArchive_Serialize_BitfieldBool(Ar, BuildSettings.bBuildReversedIndexBuffer);
	FArchive_Serialize_BitfieldBool(Ar, BuildSettings.bUseHighPrecisionTangentBasis);
	FArchive_Serialize_BitfieldBool(Ar, BuildSettings.bUseFullPrecisionUVs);
	FArchive_Serialize_BitfieldBool(Ar, BuildSettings.bUseBackwardsCompatibleF16TruncUVs);
	FArchive_Serialize_BitfieldBool(Ar, BuildSettings.bGenerateLightmapUVs);

	Ar << BuildSettings.MinLightmapResolution;
	Ar << BuildSettings.SrcLightmapIndex;
	Ar << BuildSettings.DstLightmapIndex;

	if (Ar.IsLoading() && Ar.UEVer() < VER_UE4_BUILD_SCALE_VECTOR)
	{
		float BuildScale(1.0f);
		Ar << BuildScale;
		BuildSettings.BuildScale3D = FVector( BuildScale );
	}
	else
	{
		Ar << BuildSettings.BuildScale3D;
	}
	
	Ar << BuildSettings.DistanceFieldResolutionScale;
	FArchive_Serialize_BitfieldBool(Ar, BuildSettings.bGenerateDistanceFieldAsIfTwoSided);

	FString ReplacementMeshName = BuildSettings.DistanceFieldReplacementMesh->GetPathName();
	Ar << ReplacementMeshName;
	Ar << BuildSettings.MaxLumenMeshCards;
}

const FString& GetStaticMeshDerivedDataVersion()
{
	static FString CachedVersionString;
	if (CachedVersionString.IsEmpty())
	{
		// Static mesh versioning is controlled by the version reported by the mesh utilities module.
		IMeshUtilities& MeshUtilities = FModuleManager::Get().LoadModuleChecked<IMeshUtilities>(TEXT("MeshUtilities"));
		CachedVersionString = FString::Printf(TEXT("%s_%s_%s"),
			*FDevSystemGuids::GetSystemGuid(FDevSystemGuids::Get().STATICMESH_DERIVEDDATA_VER).ToString(),
			*MeshUtilities.GetVersionString(),
			*Nanite::IBuilderModule::Get().GetVersionString()
			);
	}
	return CachedVersionString;
}

class FStaticMeshStatusMessageContext : public FScopedSlowTask
{
public:
	explicit FStaticMeshStatusMessageContext(const FText& InMessage)
		: FScopedSlowTask(0, InMessage)
	{
		UE_LOGF(LogStaticMesh,Display,"%ls",*InMessage.ToString());
		MakeDialog();
	}
};

namespace StaticMeshDerivedDataTimings
{
	int64 GetCycles = 0;
	int64 BuildCycles = 0;
	int64 ConvertCycles = 0;

	static void DumpTimings()
	{
		UE_LOGF(LogStaticMesh,Log,"Derived Data Times: Get=%.3fs Build=%.3fs ConvertLegacy=%.3fs",
			FPlatformTime::ToSeconds(GetCycles),
			FPlatformTime::ToSeconds(BuildCycles),
			FPlatformTime::ToSeconds(ConvertCycles)
			);
	}

	static FAutoConsoleCommand DumpTimingsCmd(
		TEXT("sm.DerivedDataTimings"),
		TEXT("Dumps derived data timings to the log."),
		FConsoleCommandDelegate::CreateStatic(DumpTimings)
		);
}

static FString BuildStaticMeshDerivedDataKey(const FString& KeySuffix)
{
	return FDerivedDataCacheInterface::BuildCacheKey(
		TEXT("STATICMESH"),
		*GetStaticMeshDerivedDataVersion(),
		*KeySuffix);
}

static FString BuildStaticMeshDerivedDataKeySuffix(const ITargetPlatform* TargetPlatform, UStaticMesh* Mesh, const FStaticMeshLODGroup& LODGroup)
{
	FString KeySuffix;
	TArray<uint8> TempBytes;
	TempBytes.Reserve(64);

	// Add LightmapUVVersion to key going forward
	if ( (ELightmapUVVersion)Mesh->GetLightmapUVVersion() > ELightmapUVVersion::BitByBit )
	{
		KeySuffix += LexToString(Mesh->GetLightmapUVVersion());
	}
#if WITH_EDITOR
	if (GIsAutomationTesting && Mesh->BuildCacheAutomationTestGuid.IsValid())
	{
		//If we are in automation testing and the BuildCacheAutomationTestGuid was set
		KeySuffix += Mesh->BuildCacheAutomationTestGuid.ToString(EGuidFormats::Digits);
	}
#endif

	if (Mesh->IsNaniteEnabled())
	{
		TempBytes.Reset();
		FMemoryWriter Ar(TempBytes, /*bIsPersistent=*/ true);
		SerializeNaniteSettingsForDDC(Ar, Mesh->GetNaniteSettings(), Mesh->IsNaniteForceEnabled());

		if (Mesh->IsNaniteAssembly() && NaniteAssembliesSupported())
		{
			// include a hash of the references' DDC keys to invalidate the assembly when its references change
			check(Mesh->HasCachedNaniteAssemblyReferences());

			TMemoryHasher<FXxHash64Builder, FXxHash64> HashAr;
			HashAr.SetIsPersistent(true);
		
			for (UStaticMesh* Reference : Mesh->GetCachedNaniteAssemblyReferences())
			{
				if (Reference != nullptr)
				{
					const FStaticMeshLODGroup& RefLODGroup = TargetPlatform->GetStaticMeshLODSettings().GetLODGroup(Reference->GetLODGroup());
					FString RefKey = BuildStaticMeshDerivedDataKey(BuildStaticMeshDerivedDataKeySuffix(TargetPlatform, Reference, RefLODGroup));
					HashAr << RefKey;
				}
				else
				{
					static FString MissingKey = "!!!MISSING!!!";
					HashAr << MissingKey;
				}
			}
			
			uint64 AssemblyDataHash = HashAr.Finalize().Hash;
			Ar << AssemblyDataHash;
		}

		const uint8* SettingsAsBytes = TempBytes.GetData();
		KeySuffix.Reserve(KeySuffix.Len() + TempBytes.Num() + 1);
		for (int32 ByteIndex = 0; ByteIndex < TempBytes.Num(); ++ByteIndex)
		{
			ByteToHex(SettingsAsBytes[ByteIndex], KeySuffix);
		}

		if (ShouldStripNaniteFallbackMesh(TargetPlatform, Mesh))
		{
			KeySuffix += TEXT("NFMSV2");
		}

		if (ShouldUseSourceMeshForUVDensity(Mesh->GetNaniteSettings()))
		{
			KeySuffix += TEXT("_SMUVD_");
		}

		{
			// Value of this CVar may effect adaptive tessellation
			static const TConsoleVariableData<int32>* CVar = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.Nanite.AdaptiveTessellation.Mode"));
			if (CVar)
			{
				const int32 Mode = CVar->GetValueOnAnyThread();
				if (Mode != 0)
				{
					// no invalidation needed for mode 0, which corresponds to behavior before CVar was introduced
					KeySuffix += FString::Printf(TEXT("_AT%d"), CVar->GetValueOnAnyThread());
				}
			}
		}
	}

	int32 NumLODs = Mesh->GetNumSourceModels();
	bool bHasNonUniformBuildScale = false;
	for (int32 LODIndex = 0; LODIndex < NumLODs; ++LODIndex)
	{
		FStaticMeshSourceModel& SrcModel = Mesh->GetSourceModel(LODIndex);
		if (!SrcModel.BuildSettings.BuildScale3D.AllComponentsEqual())
		{
			bHasNonUniformBuildScale = true;
		}
		
		check(SrcModel.RawMeshBulkData->IsEmpty());
		const bool bMeshDescriptionValid = !SrcModel.GetMeshDescriptionBulkData()->IsEmpty();
		if (bMeshDescriptionValid)
		{
			KeySuffix += SrcModel.GetMeshDescriptionBulkData()->GetUnversionedIdString();
		}
		else
		{
			// If mesh description bulk data is empty, this is a generated LOD
			KeySuffix += "_";
		}

		// Serialize the build and reduction settings into a temporary array. The archive
		// is flagged as persistent so that machines of different endianness produce
		// identical binary results.
		TempBytes.Reset();
		FMemoryWriter Ar(TempBytes, /*bIsPersistent=*/ true);

		if (LODIndex == 0 || bMeshDescriptionValid)
		{
			// Only include the build settings into the DDC key for LOD that will use them (generated LOD use the base model's)
			SerializeBuildSettingsForDDC(Ar, SrcModel.BuildSettings);
		}

		ANSICHAR Flag[2] = { SrcModel.BuildSettings.bUseFullPrecisionUVs ? '1' : '0', '\0' };
		Ar.Serialize(Flag, 1);

		FMeshReductionSettings FinalReductionSettings = LODGroup.GetSettings(SrcModel.ReductionSettings, LODIndex);
		SerializeReductionSettingsForDDC(Ar, FinalReductionSettings);

		// Now convert the raw bytes to a string.
		const uint8* SettingsAsBytes = TempBytes.GetData();
		KeySuffix.Reserve(KeySuffix.Len() + TempBytes.Num() + 1);
		for (int32 ByteIndex = 0; ByteIndex < TempBytes.Num(); ++ByteIndex)
		{
			ByteToHex(SettingsAsBytes[ByteIndex], KeySuffix);
		}
	}

	// Note: this ifdef is for consistency/generality but this whole function is part of a giant multi-thousand-line editor-only block
#if WITH_EDITORONLY_DATA
	if (bHasNonUniformBuildScale) // intentionally only affect key suffix if there is actual non-uniform scaling; otherwise legacy tangent scaling has no effect
	{
		KeySuffix += "LTS";
		KeySuffix.AppendChar(Mesh->GetLegacyTangentScaling() ? TEXT('1') : TEXT('0'));
	}
#endif

	// Add hi-res mesh description into DDC key (only on platforms that build from it)
	if (DoesTargetPlatformSupportNanite(TargetPlatform) && !Mesh->GetHiResSourceModel().GetMeshDescriptionBulkData()->IsEmpty())
	{
		KeySuffix += Mesh->GetHiResSourceModel().GetMeshDescriptionBulkData()->GetUnversionedIdString();
	}
	else
	{
		KeySuffix += "_";
	}

	if (Mesh->bSupportRayTracing && TargetPlatform->UsesRayTracing())
	{
		KeySuffix += TEXT("RT1");

		if (IsRayTracingEnableOnDemandSupported())
		{
			KeySuffix += TEXT("_RBR");
		}
		
		if (CVarStaticMeshRayTracingProxies.GetValueOnAnyThread() && (Mesh->GetRayTracingProxySettings().bEnabled || ShouldGenerateRayTracingProxiesByDefault()))
		{
			TempBytes.Reset();
			FMemoryWriter Ar(TempBytes, /*bIsPersistent=*/ true);
			SerializeRayTracingProxySettingsForDDC(Ar, Mesh->GetRayTracingProxySettings());

			const uint8* SettingsAsBytes = TempBytes.GetData();
			KeySuffix.Reserve(KeySuffix.Len() + TempBytes.Num() + 1);
			for (int32 ByteIndex = 0; ByteIndex < TempBytes.Num(); ++ByteIndex)
			{
				ByteToHex(SettingsAsBytes[ByteIndex], KeySuffix);
			}

			const int32 LOD1MaxNumOfTriangles = CVarStaticMeshRayTracingProxiesLOD1MaxNumOfTriangles.GetValueOnAnyThread();
			KeySuffix += FString::Printf(TEXT("_LOD1(T%d)"), LOD1MaxNumOfTriangles);
		}
	}
	else
	{
		KeySuffix += TEXT("RT0");
	}

	KeySuffix.AppendChar(Mesh->bSupportUniformlyDistributedSampling ? TEXT('1') : TEXT('0'));

	// Value of this CVar affects index buffer <-> painted vertex color correspondence (see UE-51421).
	static const TConsoleVariableData<int32>* CVar = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.TriangleOrderOptimization"));

	// depending on module loading order this might be called too early on Linux (possibly other platforms too?)
	if (CVar == nullptr)
	{
		FModuleManager::Get().LoadModuleChecked<IMeshUtilities>(TEXT("MeshUtilities"));
		CVar = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.TriangleOrderOptimization"));
	}

	if (CVar)
	{
		switch (CVar->GetValueOnAnyThread())
		{
			case 2:
				KeySuffix += TEXT("_NoTOO");
				break;
			case 0:
				KeySuffix += TEXT("_NVTS");
				break;
			case 1:
				// intentional - default value will not influence DDC to avoid unnecessary invalidation
				break;
			default:
				KeySuffix += FString::Printf(TEXT("_TOO%d"), CVar->GetValueOnAnyThread());	//	 allow unknown values transparently
				break;
		}
	}

	IMeshBuilderModule::GetForPlatform(TargetPlatform).AppendToDDCKey(KeySuffix, false);

	if (TargetPlatform->GetPlatformInfo().PlatformGroupName == TEXT("Desktop")
		&& CVarStripMinLodDataDuringCooking.GetValueOnAnyThread() != 0
		&& CVarStaticMeshKeepMobileMinLODSettingOnDesktop.GetValueOnAnyThread() != 0)
	{
		KeySuffix += TEXT("_MinMLOD");
	}

	// Append the section material slot mappings for LOD0, as they are baked into the Nanite build.	
	const FMeshSectionInfoMap& SectionInfoMap = Mesh->GetSectionInfoMap();
	int32 NumLOD0Sections = SectionInfoMap.GetSectionNumber(0);
	KeySuffix += TEXT("_");
	for (int32 SectionIndex = 0; SectionIndex < NumLOD0Sections; SectionIndex++)
	{
		KeySuffix += LexToString(SectionInfoMap.Get(0, SectionIndex).MaterialIndex);
	}

#if PLATFORM_CPU_ARM_FAMILY
	// Separate out arm keys as x64 and arm64 clang do not generate the same data for a given
	// input. Add the arm specifically so that a) we avoid rebuilding the current DDC and
	// b) we can remove it once we get arm64 to be consistent.
	KeySuffix.Append(TEXT("_arm64"));
#endif

	// Settings for optional index buffers.
	const int32 bDepthOnlyIndexBuffers = CVarSupportDepthOnlyIndexBuffers.GetValueOnAnyThread() ? 1 : 0;
	const int32 bReversedIndexBuffers = CVarSupportReversedIndexBuffers.GetValueOnAnyThread() ? 1 : 0;

	KeySuffix += FString::Printf(TEXT("_IB%x"), (bDepthOnlyIndexBuffers << 0) | (bReversedIndexBuffers << 1));

	return KeySuffix;
}

namespace UE::Private::StaticMesh
{

#if WITH_EDITOR
FString BuildStaticMeshDerivedDataKey(const ITargetPlatform* TargetPlatform, UStaticMesh* Mesh, const FStaticMeshLODGroup& LODGroup)
{
	return BuildStaticMeshDerivedDataKey(BuildStaticMeshDerivedDataKeySuffix(TargetPlatform, Mesh, LODGroup));
}
#endif

} // UE::Private::StaticMesh

static FString BuildStaticMeshLODDerivedDataKey(const FString& KeySuffix, int32 LODIdx)
{
	return FDerivedDataCacheInterface::BuildCacheKey(
		TEXT("STATICMESH"),
		*GetStaticMeshDerivedDataVersion(),
		*FString::Printf(TEXT("%s_LOD%d"), *KeySuffix, LODIdx));
}

void FStaticMeshRenderData::ComputeUVDensities(UStaticMesh* Mesh)
{
#if WITH_EDITORONLY_DATA
	for (int32 LODIndex = 0; LODIndex < LODResources.Num(); ++LODIndex)
	{
		FStaticMeshLODResources& LODModel = LODResources[LODIndex];
		const int32 NumTexCoords = FMath::Min<int32>(LODModel.GetNumTexCoords(), MAX_STATIC_TEXCOORDS);

		for (FStaticMeshSection& SectionInfo : LODModel.Sections)
		{
			FMemory::Memzero(SectionInfo.UVDensities);
			FMemory::Memzero(SectionInfo.Weights);

			bool bComputedFromSourceModel = false;
			if (LODIndex == 0 && ShouldUseSourceMeshForUVDensity(Mesh->GetNaniteSettings()))
			{
				if (FMeshDescription* MeshDescription = Mesh->GetMeshDescription(0))
				{
					const TArray<FStaticMaterial>& MeshMaterials = Mesh->GetStaticMaterials();
					if (MeshMaterials.IsValidIndex(SectionInfo.MaterialIndex))
					{
						const FName MaterialSlotName = MeshMaterials[SectionInfo.MaterialIndex].ImportedMaterialSlotName;
						FUVDensityAccumulatorSourceMesh Accumulator(*MeshDescription);
						bComputedFromSourceModel |= Accumulator.AccumulateDensitiesForMaterial(
							MaterialSlotName,
							MakeArrayView(SectionInfo.UVDensities, UE_ARRAY_COUNT(SectionInfo.UVDensities)),
							MakeArrayView(SectionInfo.Weights, UE_ARRAY_COUNT(SectionInfo.Weights)));
					}
				}

				if (Mesh->IsNaniteAssembly())
				{
					check(Mesh->HasCachedNaniteAssemblyReferences());
					const TArray<FNaniteAssemblyPart>& AssemblyParts = Mesh->GetNaniteSettings().NaniteAssemblyData.Parts;
					for (int32 PartIndex = 0; PartIndex < AssemblyParts.Num(); ++PartIndex)
					{
						UStaticMesh* PartMesh = Mesh->GetCachedNaniteAssemblyReferences()[PartIndex];
						if (PartMesh == nullptr)
						{
							continue;
						}

						FMeshDescription* MeshDescription = PartMesh->GetMeshDescription(0);
						if (MeshDescription == nullptr)
						{
							continue;
						}

						const TArray<FStaticMaterial>& PartMaterials = PartMesh->GetStaticMaterials();
						const TArray<int32>& MaterialRemap = AssemblyParts[PartIndex].MaterialRemap;
						TArray<int32, TInlineAllocator<4>> PartMaterialIndices;
						if (MaterialRemap.Num() > 0)
						{
							for (int32 i = 0, N = FMath::Min(PartMaterials.Num(), MaterialRemap.Num()); i < N; ++i)
							{
								if (MaterialRemap[i] == SectionInfo.MaterialIndex)
								{
									PartMaterialIndices.Add(i);
								}
							}
						}
						else if (PartMaterials.IsValidIndex(SectionInfo.MaterialIndex))
						{
							PartMaterialIndices.Add(SectionInfo.MaterialIndex);
						}

						if (PartMaterialIndices.Num() == 0)
						{
							continue;
						}

						TArray<float> InstanceScales;
						for (const FNaniteAssemblyNode& Node : Mesh->GetNaniteSettings().NaniteAssemblyData.Nodes)
						{
							if (Node.PartIndex == PartIndex)
							{
								InstanceScales.Add(Node.Transform.GetScale3D().GetMax());
							}
						}

						if (InstanceScales.Num() == 0)
						{
							continue;
						}
						
						FUVDensityAccumulatorSourceMesh Accumulator(*MeshDescription, MoveTemp(InstanceScales));
						for (int32 MaterialIndexInPart : PartMaterialIndices)
						{
							if (PartMaterials.IsValidIndex(MaterialIndexInPart))
							{
								const FName MaterialSlotName = PartMaterials[MaterialIndexInPart].ImportedMaterialSlotName;
								bComputedFromSourceModel |= Accumulator.AccumulateDensitiesForMaterial(
									MaterialSlotName,
									MakeArrayView(SectionInfo.UVDensities, UE_ARRAY_COUNT(SectionInfo.UVDensities)),
									MakeArrayView(SectionInfo.Weights, UE_ARRAY_COUNT(SectionInfo.Weights)));
							}
						}
					}
				}
			}
			
			if (!bComputedFromSourceModel)
			{
				FUVDensityAccumulator UVDensityAccs[MAX_STATIC_TEXCOORDS];
				for (int32 UVIndex = 0; UVIndex < NumTexCoords; ++UVIndex)
				{
					UVDensityAccs[UVIndex].Reserve(SectionInfo.NumTriangles);
				}

				FIndexArrayView IndexBuffer = LODModel.IndexBuffer.GetArrayView();

				for (uint32 TriangleIndex = 0; TriangleIndex < SectionInfo.NumTriangles; ++TriangleIndex)
				{
					const int32 Index0 = IndexBuffer[SectionInfo.FirstIndex + TriangleIndex * 3 + 0];
					const int32 Index1 = IndexBuffer[SectionInfo.FirstIndex + TriangleIndex * 3 + 1];
					const int32 Index2 = IndexBuffer[SectionInfo.FirstIndex + TriangleIndex * 3 + 2];

					const float Area = FUVDensityAccumulator::GetTriangleArea(
						LODModel.VertexBuffers.PositionVertexBuffer.VertexPosition(Index0),
						LODModel.VertexBuffers.PositionVertexBuffer.VertexPosition(Index1),
						LODModel.VertexBuffers.PositionVertexBuffer.VertexPosition(Index2));

					if (Area > UE_SMALL_NUMBER)
					{
						for (int32 UVIndex = 0; UVIndex < NumTexCoords; ++UVIndex)
						{
							const float UVArea = FUVDensityAccumulator::GetUVChannelArea(
								LODModel.VertexBuffers.StaticMeshVertexBuffer.GetVertexUV(Index0, UVIndex),
								LODModel.VertexBuffers.StaticMeshVertexBuffer.GetVertexUV(Index1, UVIndex),
								LODModel.VertexBuffers.StaticMeshVertexBuffer.GetVertexUV(Index2, UVIndex));

							UVDensityAccs[UVIndex].PushTriangle(Area, UVArea);
						}
					}
				}

				for (int32 UVIndex = 0; UVIndex < NumTexCoords; ++UVIndex)
				{
					UVDensityAccs[UVIndex].AccumulateDensity(SectionInfo.UVDensities[UVIndex], SectionInfo.Weights[UVIndex]);
				}
			}

			// Normalize the densities
			for (int32 UVIndex = 0; UVIndex < NumTexCoords; ++UVIndex)
			{
				if (SectionInfo.Weights[UVIndex] > UE_SMALL_NUMBER)
				{
					SectionInfo.UVDensities[UVIndex] /= SectionInfo.Weights[UVIndex];
				}
			}
		}
	}
#endif // WITH_EDITORONLY_DATA
}

void FStaticMeshRenderData::BuildAreaWeighedSamplingData()
{
	for (FStaticMeshLODResources& LODModel : LODResources)
	{
		for (FStaticMeshSection& SectionInfo : LODModel.Sections)
		{
			LODModel.AreaWeightedSectionSamplers.SetNum(LODModel.Sections.Num());
			for (int32 i = 0; i < LODModel.Sections.Num(); ++i)
			{
				LODModel.AreaWeightedSectionSamplers[i].Init(&LODModel, i);
			}
			LODModel.AreaWeightedSampler.Init(&LODModel);
		}
	}
}

UE_TRACE_EVENT_BEGIN(Cpu, StaticMeshCache, NoSync)
UE_TRACE_EVENT_FIELD(UE::Trace::WideString, KeySuffix)
UE_TRACE_EVENT_END()

void FStaticMeshRenderData::Cache(const ITargetPlatform* TargetPlatform, UStaticMesh* Owner, const FStaticMeshLODSettings& LODSettings)
{
	UE_LOG_CONTEXT("StaticMesh", FSoftObjectPath(Owner));
	if (Owner->GetOutermost()->bIsCookedForEditor)
	{
		// Don't cache for cooked packages
		return;
	}

	if (Owner->IsNaniteAssembly() && !NaniteAssembliesSupported())
	{
		// Pop up a notification to warn the user about assembly support
		ShowNaniteAssembliesDisabledWarningNotification();
	}

	check(NaniteResourcesPtr.IsValid());
	Nanite::FResources& NaniteResources = *NaniteResourcesPtr.Get();

	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FStaticMeshRenderData::Cache);
		LLM_SCOPE_BYNAME(TEXT("AssetCompilation/StaticMesh"));

		COOK_STAT(auto Timer = StaticMeshCookStats::UsageStats.TimeSyncWork());
		double T0 = FPlatformTime::Seconds();
		int32 NumLODs = Owner->GetNumSourceModels();
		const FStaticMeshLODGroup& LODGroup = LODSettings.GetLODGroup(Owner->GetLODGroup());
		const FString KeySuffix = BuildStaticMeshDerivedDataKeySuffix(TargetPlatform, Owner, LODGroup);
#if CPUPROFILERTRACE_ENABLED
		UE_TRACE_LOG_SCOPED_T(Cpu, StaticMeshCache, CpuChannel)
			<< StaticMeshCache.KeySuffix(*KeySuffix);
#endif
		UE_LOG_CONTEXT("StaticMeshKeySuffix", KeySuffix);
		FString NewDerivedDataKey = BuildStaticMeshDerivedDataKey(KeySuffix);
		if (NumCacheAttempts == 0 || !IsRunningCookCommandlet())
		{
			DerivedDataKey = MoveTemp(NewDerivedDataKey);
		}
		else if (DerivedDataKey != NewDerivedDataKey)
		{
			UE_LOGF(LogStaticMesh, Error,
				"Indeterminism in FStaticMeshRenderData::Cache call to BuildStaticMeshDerivedDataKey. Using the previously computed RenderData even though the key is now different." "\n\tStaticMesh=%ls, Platform=%ls" "\n\tPreviousKey=%ls" "\n\tCurrentKey =%ls",
				*Owner->GetPathName(), *TargetPlatform->PlatformName(),
				*DerivedDataKey, *NewDerivedDataKey);
		}
		++NumCacheAttempts;

		using namespace UE;
		using namespace UE::DerivedData;

		static const FValueId MeshDataId = FValueId::FromName("MeshData");
		static const FValueId NaniteStreamingDataId = FValueId::FromName("NaniteStreamingData");
		static const double DDCSizeToEstimateFactor = 0.9;	// Hack: DDC uses fast compression. Compressed data is typically at least 10% smaller in shipped build, so use that as our estimate.

		FCacheKey CacheKey;
		CacheKey.Bucket = FCacheBucket(TEXT("StaticMesh"));
		CacheKey.Hash = FIoHash::HashBuffer(MakeMemoryView(FTCHARToUTF8(DerivedDataKey)));

		uint64 EstimatedMeshDataCompressedSize = 0;
		FSharedBuffer MeshDataBuffer;
		FIoHash NaniteStreamingDataHash;

		bool bTryLoadingFromDDC = !NaniteResources.HasBuildFromDDCError();
		if (bTryLoadingFromDDC)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(GetDDC);

			FCacheRecordPolicyBuilder PolicyBuilder(ECachePolicy::Default | ECachePolicy::KeepAlive);
			PolicyBuilder.AddValuePolicy(NaniteStreamingDataId, ECachePolicy::Default | ECachePolicy::SkipData);
			
			FCacheGetRequest Request;
			Request.Name = Owner->GetPathName();
			Request.Key = CacheKey;
			Request.Policy = PolicyBuilder.Build();

			FRequestOwner RequestOwner(EPriority::Blocking);
			GetCache().Get(MakeArrayView(&Request, 1), RequestOwner,
				[&MeshDataBuffer, &EstimatedMeshDataCompressedSize, &NaniteStreamingDataHash](FCacheGetResponse&& Response)
				{
					if(Response.Status == EStatus::Ok)
					{
						const FCompressedBuffer& CompressedBuffer = Response.Record.GetValue(MeshDataId).GetData();
						MeshDataBuffer = CompressedBuffer.Decompress();
						EstimatedMeshDataCompressedSize = uint64(CompressedBuffer.GetCompressedSize() * DDCSizeToEstimateFactor);

						NaniteStreamingDataHash = Response.Record.GetValue(NaniteStreamingDataId).GetRawHash();
					}
				});
			RequestOwner.Wait();
		}

		if (UE::Tasks::FCancellationTokenScope::IsCurrentWorkCanceled())
		{
			return;
		}

		if (!MeshDataBuffer.IsNull())
		{
			COOK_STAT(Timer.AddHit(MeshDataBuffer.GetSize()));

			FMemoryReaderView Ar(MeshDataBuffer.GetView(), /*bIsPersistent=*/ true);
			Serialize(Ar, Owner, /*bCooked=*/ false);

			// Reconstruct EstimatedCompressedSize
			// It is not serialized as it is not known before the serialized data has been compressed and we don't want to compress twice.
			EstimatedCompressedSize = EstimatedMeshDataCompressedSize + EstimatedNaniteStreamingCompressedSize;

			check(NaniteResources.StreamablePages.GetBulkDataSize() == 0);
			if (NaniteResources.ResourceFlags & NANITE_RESOURCE_FLAG_STREAMING_DATA_IN_DDC)
			{
				NaniteResources.DDCKeyHash = CacheKey.Hash;
				NaniteResources.DDCRawHash = NaniteStreamingDataHash;
			}

			for (int32 LODIdx = 0; LODIdx < LODResources.Num(); ++LODIdx)
			{
				FStaticMeshLODResources& LODResource = LODResources[LODIdx];
				if (LODResource.bBuffersInlined)
				{
					break;
				}
				// TODO: can we postpone the loading to streaming time?
				LODResource.DerivedDataKey = BuildStaticMeshLODDerivedDataKey(KeySuffix, LODIdx);
				typename FStaticMeshLODResources::FStaticMeshBuffersSize DummyBuffersSize;
				LODResource.SerializeBuffers(Ar, Owner, 0, DummyBuffersSize);
				typename FStaticMeshLODResources::FStaticMeshBuffersSize LODBuffersSize;
				Ar << LODBuffersSize;
				LODResource.BuffersSize = LODBuffersSize.CalcBuffersSize();
				check(LODResource.BuffersSize == DummyBuffersSize.CalcBuffersSize());
			}

			int32 T1 = FPlatformTime::Cycles();
			UE_LOGF(LogStaticMesh,Verbose,"Static mesh found in DDC [%fms] %ls",
				FPlatformTime::ToMilliseconds(T1-T0),
				*Owner->GetPathName()
				);
			FPlatformAtomics::InterlockedAdd(&StaticMeshDerivedDataTimings::GetCycles, T1 - T0);
		}
		else
		{
			FFormatNamedArguments Args;
			Args.Add(TEXT("StaticMeshName"), FText::FromString( Owner->GetName() ) );
			Args.Add(TEXT("EstimatedMemory"), FText::FromString( FString::SanitizeFloat(double(Owner->GetBuildRequiredMemoryEstimate()) / (1024.0 * 1024.0), 3) ));
			FStaticMeshStatusMessageContext StatusContext( FText::Format( NSLOCTEXT("Engine", "BuildingStaticMeshStatus", "Building static mesh {StaticMeshName} (Required Memory Estimate: {EstimatedMemory} MB)..."), Args ) );

			checkf(!Owner->HasAnyFlags(RF_NeedLoad), TEXT("StaticMesh %s being PostLoaded before having been serialized - this suggests an async loading problem."), *GetPathNameSafe(Owner));
			if (!Owner->IsMeshDescriptionValid(0))
			{
				UE_LOGF(LogStaticMesh, Error, "Bad MeshDescription on %ls", *GetPathNameSafe(Owner));
				return;
			}

			if (Owner->bDoFastBuild)
			{
				// If the mesh is built via the FastBuild path, just build it each time here instead of using the DDC as a cache.
				const int32 NumSourceModels = Owner->GetNumSourceModels();
				AllocateLODResources(NumSourceModels);
				for (int32 LodIndex = 0; LodIndex < NumSourceModels; LodIndex++)
				{
					if (Owner->IsMeshDescriptionValid(LodIndex))
					{
						if (UE::Tasks::FCancellationTokenScope::IsCurrentWorkCanceled())
						{
							return;
						}

						Owner->BuildFromMeshDescription(*Owner->GetMeshDescription(LodIndex), LODResources[LodIndex]);
					}
					else
					{
						UE_LOGF(LogStaticMesh, Error, "Bad MeshDescription at lod index %d on %ls", LodIndex, *GetPathNameSafe(Owner));
					}
				}

#if RHI_RAYTRACING
				if (Owner->bSupportRayTracing && TargetPlatform->UsesRayTracing())
				{
					InitializeRayTracingRepresentationFromRenderingLODs();
				}
#endif // RHI_RAYTRACING
			}
			else
			{
				IMeshBuilderModule& MeshBuilderModule = IMeshBuilderModule::GetForPlatform(TargetPlatform);

				if (!MeshBuilderModule.BuildMesh(*this, FStaticMeshBuildParameters(Owner, TargetPlatform, LODGroup)))
				{
					if (UE::Tasks::FCancellationTokenScope::IsCurrentWorkCanceled())
					{
						return;
					}

					UE_LOGF(LogStaticMesh, Error, "Failed to build static mesh. See previous line(s) for details.");
					return;
				}

				ComputeUVDensities(Owner);

				if (UE::Tasks::FCancellationTokenScope::IsCurrentWorkCanceled())
				{
					return;
				}

				if(Owner->bSupportUniformlyDistributedSampling)
				{
					BuildAreaWeighedSamplingData();
				}
			}
			
			int64 TotalPushedBytes = 0;
			FCacheRecordBuilder RecordBuilder(CacheKey);
			if (NaniteResources.PageStreamingStates.Num() > 0)
			{
				if (NaniteResources.HasStreamingData())
				{
					// Compress streaming data and add it to record builder
					FByteBulkData& BulkData = NaniteResources.StreamablePages;
					TotalPushedBytes += BulkData.GetBulkDataSize();

					FValue Value = FValue::Compress(FSharedBuffer::MakeView(BulkData.LockReadOnly(), BulkData.GetBulkDataSize()));
					RecordBuilder.AddValue(NaniteStreamingDataId, Value);
					EstimatedNaniteStreamingCompressedSize = uint64(Value.GetData().GetCompressedSize() * DDCSizeToEstimateFactor);
					BulkData.Unlock();
					NaniteResources.ResourceFlags |= NANITE_RESOURCE_FLAG_STREAMING_DATA_IN_DDC;
					NaniteResources.DDCKeyHash = CacheKey.Hash;
					NaniteResources.DDCRawHash = Value.GetRawHash();
				}
				
				// Compress non-streaming data for size estimation
				FLargeMemoryWriter Ar(0, /*bIsPersistent=*/ true);
				NaniteResources.Serialize(Ar, Owner, /*bCooked=*/ false);
				FCompressedBuffer CompressedBuffer = FCompressedBuffer::Compress(FSharedBuffer::MakeView(Ar.GetData(), Ar.TotalSize()));
				EstimatedNaniteTotalCompressedSize = EstimatedNaniteStreamingCompressedSize + uint64(CompressedBuffer.GetCompressedSize() * DDCSizeToEstimateFactor);

				bHasNaniteFallbackMesh = true;
			}
			
			if (UE::Tasks::FCancellationTokenScope::IsCurrentWorkCanceled())
			{
				return;
			}

			bLODsShareStaticLighting = Owner->CanLODsShareStaticLighting();
			FLargeMemoryWriter Ar(0, /*bIsPersistent=*/ true);
			Serialize(Ar, Owner, /*bCooked=*/ false);

			for (int32 LODIdx = 0; LODIdx < LODResources.Num(); ++LODIdx)
			{
				FStaticMeshLODResources& LODResource = LODResources[LODIdx];
				if (LODResource.bBuffersInlined)
				{
					break;
				}
				typename FStaticMeshLODResources::FStaticMeshBuffersSize LODBuffersSize;
				const uint8 LODStripFlags = FStaticMeshLODResources::GenerateClassStripFlags(Ar, Owner, LODIdx);
				LODResource.SerializeBuffers(Ar, Owner, LODStripFlags, LODBuffersSize);
				Ar << LODBuffersSize;
				LODResource.DerivedDataKey = BuildStaticMeshLODDerivedDataKey(KeySuffix, LODIdx);
				// TODO: Save non-inlined LODs separately
			}

			bool bSaveDDC = true;
#if WITH_EDITOR
			//Do not save ddc when we are forcing the regeneration of ddc in automation test
			//No need to take more space in the ddc.
			if (GIsAutomationTesting && Owner->BuildCacheAutomationTestGuid.IsValid() && !NaniteResources.HasStreamingData())
			{
				bSaveDDC = false;
			}
#endif
			bool bSavedToDDC = false;
			if (bSaveDDC)
			{
				TRACE_CPUPROFILER_EVENT_SCOPE(SaveDDC);

				bool bInvalidateNaniteStreaming = false;
				FValue Value = FValue::Compress(FSharedBuffer::MakeView(Ar.GetData(), Ar.TotalSize()));
				RecordBuilder.AddValue(MeshDataId, Value);
				TotalPushedBytes += Ar.TotalSize();
				EstimatedMeshDataCompressedSize = uint64(Value.GetData().GetCompressedSize() * DDCSizeToEstimateFactor);

				FCacheRecordPolicyBuilder PolicyBuilder(ECachePolicy::Default | ECachePolicy::SkipData | ECachePolicy::KeepAlive);
				PolicyBuilder.AddValuePolicy(MeshDataId, ECachePolicy::Default);

				FRequestOwner RequestOwner(UE::DerivedData::EPriority::Blocking);
				const FCachePutRequest PutRequest = { FSharedString(Owner->GetPathName()), RecordBuilder.Build(), PolicyBuilder.Build() };
				GetCache().Put(MakeArrayView(&PutRequest, 1), RequestOwner,
					[&bSavedToDDC, &bInvalidateNaniteStreaming, &PutRequest, &MeshDataBuffer,
					 &EstimatedMeshDataCompressedSize, &NaniteStreamingDataHash](FCachePutResponse&& Response)
					{
						if (Response.Status == EStatus::Ok)
						{
							bSavedToDDC = true;

							const FValueWithId& ResponseMeshValue = Response.Record.GetValue(MeshDataId);
							const FValueWithId& ResponseNaniteStreamingValue = Response.Record.GetValue(NaniteStreamingDataId);

							if (ResponseMeshValue != PutRequest.Record.GetValue(MeshDataId))
							{
								// The request and response differ, so we're going to reserialize with what is actually in the cache
								const FCompressedBuffer& CompressedBuffer = ResponseMeshValue.GetData();
								MeshDataBuffer = CompressedBuffer.Decompress();
								EstimatedMeshDataCompressedSize = uint64(CompressedBuffer.GetCompressedSize() * DDCSizeToEstimateFactor);
							}

							NaniteStreamingDataHash = ResponseNaniteStreamingValue.GetRawHash();
							if (ResponseNaniteStreamingValue != PutRequest.Record.GetValue(NaniteStreamingDataId))
							{
								bInvalidateNaniteStreaming = true;
							}
						}
					});

				RequestOwner.Wait();

				if (bSavedToDDC)
				{
					NaniteResources.SetHasBuildFromDDCError(false);
					if (bInvalidateNaniteStreaming || (NaniteResources.HasStreamingData() && !IsRunningCookCommandlet()))
					{
						// Drop streaming data from memory when it has been successfully committed to DDC or if the streaming data we have doesn't match the cache record
						NaniteResources.DropBulkData();

						if (bInvalidateNaniteStreaming)
						{
							UE_LOGFMT(LogStaticMesh, Display,
								"StaticMesh Nanite streaming data saved to DDC mismatched the data that was already in the cache. Dropping data and replacing the hash. Asset: {Asset}. Cache Key: {CacheKey}",
								UE::FAssetLog(Owner), CacheKey);
							NaniteResources.DDCRawHash = NaniteStreamingDataHash;
						}
					}

					if (!MeshDataBuffer.IsNull())
					{
						// The mesh data record in cache differs from what we tried to put, so we'll override it with what's in the cache
						UE_LOGFMT(LogStaticMesh, Display,
							"StaticMesh data saved to DDC mismatched the data that was already in the cache. Serializing from cached data. Asset: {Asset}. Cache Key: {CacheKey}",
							UE::FAssetLog(Owner), CacheKey);

						FMemoryReaderView ReadAr(MeshDataBuffer.GetView(), /*bIsPersistent=*/ true);
						Serialize(ReadAr, Owner, /*bCooked=*/ false);

						// Always reset the DDC hashes in case the presence of streaming data changed in re-serialization
						if (NaniteResources.HasStreamingData())
						{
							NaniteResources.DDCKeyHash = CacheKey.Hash;
							NaniteResources.DDCRawHash = NaniteStreamingDataHash;
						}
						else
						{
							NaniteResources.DDCKeyHash.Reset();
							NaniteResources.DDCRawHash.Reset();
						}
					}
				}

				EstimatedCompressedSize = EstimatedMeshDataCompressedSize + EstimatedNaniteStreamingCompressedSize;
			}

			if (NaniteResources.HasStreamingData() && !bSavedToDDC)
			{
				// Streaming data was not pushed to DDC. Disable DDC streaming flag.
				check(NaniteResources.StreamablePages.GetBulkDataSize() > 0);
				NaniteResources.ResourceFlags &= ~NANITE_RESOURCE_FLAG_STREAMING_DATA_IN_DDC;
			}

			double T1 = FPlatformTime::Seconds();
			UE_LOGF(LogStaticMesh,Log,"Built static mesh [%.2fs] %ls",
				T1-T0,
				*Owner->GetPathName()
				);
			FPlatformAtomics::InterlockedAdd(&StaticMeshDerivedDataTimings::BuildCycles, T1 - T0);
			COOK_STAT(Timer.AddMiss(TotalPushedBytes));
		}
	}

	// If the engine is exiting and is waiting on us to finish this pending task, exit asap and don't start any other work.
	if (IsEngineExitRequested())
	{
		return;
	}

	if (Owner->IsNaniteLandscape())
	{
		// Do not launch any distance field or mesh card representation tasks for this mesh
		return;
	}
	
	if (UE::Tasks::FCancellationTokenScope::IsCurrentWorkCanceled())
	{
		return;
	}

	static const auto CVar = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.GenerateMeshDistanceFields"));

	if (CVar->GetValueOnAnyThread(true) != 0 || Owner->bGenerateMeshDistanceField)
	{
		if (LODResources.IsValidIndex(0))
		{
			if (!LODResources[0].DistanceFieldData)
			{
				LODResources[0].DistanceFieldData = new FDistanceFieldVolumeData();
				LODResources[0].DistanceFieldData->AssetName = Owner->GetFName();
			}

			// Only generate distance fields and card representations for the base render data, not platform render data.
			if (this == Owner->GetRenderData())
			{
				const FMeshBuildSettings& BuildSettings = Owner->GetSourceModel(0).BuildSettings;
				UStaticMesh* MeshToGenerateFrom = BuildSettings.DistanceFieldReplacementMesh ? ToRawPtr(BuildSettings.DistanceFieldReplacementMesh) : Owner;

				if (BuildSettings.DistanceFieldReplacementMesh)
				{
					// Make sure dependency is postloaded
					BuildSettings.DistanceFieldReplacementMesh->ConditionalPostLoad();
				}

				LODResources[0].DistanceFieldData->CacheDerivedData(Owner, MeshToGenerateFrom, BuildSettings.DistanceFieldResolutionScale, BuildSettings.bGenerateDistanceFieldAsIfTwoSided);
			}
			else
			{
				// If base data is already built (implying it was loaded from DDC), copy that to platform data, otherwise
				// it will be copied later when the async queue finishes generating it.
				if (Owner->GetRenderData()->LODResources.IsValidIndex(0))
				{
					FDistanceFieldVolumeData* BaseDistanceFieldData = Owner->GetRenderData()->LODResources[0].DistanceFieldData;
					if (BaseDistanceFieldData && BaseDistanceFieldData->bAsyncBuilding == false)
					{
						*LODResources[0].DistanceFieldData = *BaseDistanceFieldData;
					}
				}
				else
				{
					UE_LOGFMT(LogStaticMesh, Error, "{StaticMesh}: Failed to fetch distance field data from LOD 0 on base render data for platform {PlatformName}", 
						FSoftObjectPath(Owner), TargetPlatform->PlatformName());
				}

				// For platform data, this either copies the base render data, or allocates a placeholder structure for data to be copied into
				// later when tasks generated by CacheDerivedData called above for the base render data completes.  For base render data (the
				// "if" case above), this call is deferred until after distance fields are built.
				BeginCacheMeshCardRepresentation(Owner, *this, FString());
			}
		}
		else
		{
			UE_LOGF(LogStaticMesh, Error, "Failed to generate distance field data for %ls due to missing LODResource for LOD 0.", *Owner->GetPathName());
		}
	}
	else
	{
		BeginCacheMeshCardRepresentation(Owner, *this, Owner->GetRenderData()->DerivedDataKey);
	}
}
#endif // #if WITH_EDITOR

FArchive& operator<<(FArchive& Ar, FStaticMaterial& Elem)
{
	Ar.UsingCustomVersion(FFortniteMainBranchObjectVersion::GUID);
	Ar.UsingCustomVersion(FRenderingObjectVersion::GUID);

	Ar << Elem.MaterialInterface;

	Ar << Elem.MaterialSlotName;
#if WITH_EDITORONLY_DATA
	if((!Ar.IsCooking() && !Ar.IsFilterEditorOnly()) || (Ar.IsCooking() && Ar.CookingTarget()->HasEditorOnlyData()))
	{
		Ar << Elem.ImportedMaterialSlotName;
	}
#endif //#if WITH_EDITORONLY_DATA

	if (!Ar.IsLoading() || Ar.CustomVer(FRenderingObjectVersion::GUID) >= FRenderingObjectVersion::TextureStreamingMeshUVChannelData)
	{
		Ar << Elem.UVChannelData;
	}

	if (!Ar.IsLoading() || Ar.CustomVer(FFortniteMainBranchObjectVersion::GUID) >= FFortniteMainBranchObjectVersion::MeshMaterialSlotOverlayMaterialAdded)
	{
		Ar << Elem.OverlayMaterialInterface;
	}
	else
	{
		Elem.OverlayMaterialInterface = nullptr;
	}
	return Ar;
}

FStaticMaterial::FStaticMaterial()
: MaterialInterface(NULL)
, MaterialSlotName(NAME_None)
#if WITH_EDITORONLY_DATA
, ImportedMaterialSlotName(NAME_None)
#endif //WITH_EDITORONLY_DATA
{

}

FStaticMaterial::FStaticMaterial(class UMaterialInterface* InMaterialInterface
, FName InMaterialSlotName
#if WITH_EDITORONLY_DATA
, FName InImportedMaterialSlotName
#endif
, UMaterialInterface* InOverlayMaterialInterface
)
: MaterialInterface(InMaterialInterface)
, MaterialSlotName(InMaterialSlotName)
#if WITH_EDITORONLY_DATA
, ImportedMaterialSlotName(InImportedMaterialSlotName)
#endif //WITH_EDITORONLY_DATA
, OverlayMaterialInterface(InOverlayMaterialInterface)
{
//If not specified add some valid material slot name
if (MaterialInterface && MaterialSlotName == NAME_None)
{
	MaterialSlotName = MaterialInterface->GetFName();
}
#if WITH_EDITORONLY_DATA
if (ImportedMaterialSlotName == NAME_None)
{
	ImportedMaterialSlotName = MaterialSlotName;
}
#endif
}

bool operator== (const FStaticMaterial& LHS, const FStaticMaterial& RHS)
{
	return (LHS.MaterialInterface == RHS.MaterialInterface &&
		LHS.MaterialSlotName == RHS.MaterialSlotName
#if WITH_EDITORONLY_DATA
		&& LHS.ImportedMaterialSlotName == RHS.ImportedMaterialSlotName
#endif
		);
}

bool operator== (const FStaticMaterial& LHS, const UMaterialInterface& RHS)
{
	return (LHS.MaterialInterface == &RHS);
}

bool operator== (const UMaterialInterface& LHS, const FStaticMaterial& RHS)
{
	return (RHS.MaterialInterface == &LHS);
}

/*-----------------------------------------------------------------------------
UStaticMesh
-----------------------------------------------------------------------------*/

#if WITH_EDITORONLY_DATA
const float UStaticMesh::MinimumAutoLODPixelError = UE_SMALL_NUMBER;
#endif	//#if WITH_EDITORONLY_DATA

UStaticMesh::UStaticMesh(const FObjectInitializer& ObjectInitializer)
	: UStreamableRenderAsset(ObjectInitializer)
#if WITH_EDITOR
	, LockedProperties((uint32)EStaticMeshAsyncProperties::None)
#endif
{
	ElementToIgnoreForTexFactor = -1;
	bHasNavigationData=true;
#if WITH_EDITORONLY_DATA
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	bAutoComputeLODScreenSize=true;
	ImportVersion = EImportStaticMeshVersion::BeforeImportStaticMeshVersionWasAdded;
	NumStreamedLODs.Default = -1;
	GetHiResSourceModel().StaticMeshDescriptionBulkData = CreateDefaultSubobject<UStaticMeshDescriptionBulkData>(TEXT("HiResMeshDescription"));
	GetHiResSourceModel().StaticMeshDescriptionBulkData->SetFlags(RF_Transactional);
	SetLegacyTangentScaling(false);
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
#endif // #if WITH_EDITORONLY_DATA
	SetLightMapResolution(4);
	SetMinLOD(0);

	bDoFastBuild = false;
	bSupportUniformlyDistributedSampling = false;

	bSupportRayTracing = true;

	bRenderingResourcesInitialized = false;
#if WITH_EDITOR
	BuildCacheAutomationTestGuid.Invalidate();
#endif
	SetQualityLevelMinLOD(0);
	MinQualityLevelLOD.SetQualityLevelCVarForCooking(GMinLodQualityLevelCVarName, GMinLodQualityLevelScalabilitySection);
}

// We don't care if the default implementation of the destructor is cleaning up 
// deprecated fields... The UObject has been properly destroyed by the garbage
// collect anyway.
PRAGMA_DISABLE_DEPRECATION_WARNINGS
UStaticMesh::~UStaticMesh()
{
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

#if WITH_EDITOR

thread_local const UStaticMesh* FStaticMeshAsyncBuildScope::StaticMeshBeingAsyncCompiled = nullptr;

void UStaticMesh::WaitUntilAsyncPropertyReleased(EStaticMeshAsyncProperties AsyncProperties) const
{
	// Async static mesh builds are only supported on editor, no-op for other builds
	if (IsCompiling() && (LockedProperties & (uint32)AsyncProperties) != 0 && FStaticMeshAsyncBuildScope::ShouldWaitOnLockedProperties(this))
	{
		TRACE_CPUPROFILER_EVENT_SCOPE_TEXT(*FString::Printf(TEXT("StaticMeshCompilationStall %s"), ToString(AsyncProperties)));
		
		if (IsInGameThread())
		{
			UE_LOGF(
				LogStaticMesh,
				Verbose,
				"Accessing property %ls of the StaticMesh while it is still being built asynchronously will force it to be compiled before continuing. "
					 "For better performance, consider making the caller async aware so it can wait until the static mesh is ready to access this property."
					 "To better understand where those calls are coming from, you can use Editor.AsyncAssetDumpStallStacks on the console.",
				ToString(AsyncProperties)
			);

			FStaticMeshCompilingManager::Get().FinishCompilation({ const_cast<UStaticMesh*>(this) });
		}
		else
		{
			// Trying to access a property from another thread that cannot force finish the compilation is invalid
			ensureMsgf(
				false,
				TEXT("Accessing property %s of the StaticMesh while it is still being built asynchronously is only supported on the game-thread. "
					 "To avoid any race-condition, consider finishing the compilation before pushing tasks to other threads or making higher-level game-thread code async aware so it "
					 "schedules the task only when the static mesh's compilation is finished. If this is a blocker, you can disable async static mesh from the editor experimental settings."),
				ToString(AsyncProperties)
			);
		}
	}
}

#endif // WITH_EDITOR

bool UStaticMesh::IsNavigationRelevant() const 
{
	return bHasNavigationData;
}

#if WITH_EDITOR

bool UStaticMesh::HasAnyDependenciesCompiling() const
{
	for (const TObjectPtr<UStaticMesh>& AssemblyRef : AssemblyReferenceCache)
	{
		if (AssemblyRef && AssemblyRef->IsCompiling())
		{
			return true;
		}
	}

	return false;
}

#endif

FStaticMeshRenderData* UStaticMesh::GetRenderData()
{
	WaitUntilAsyncPropertyReleased(EStaticMeshAsyncProperties::RenderData);

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	return RenderData.Get();
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

const FStaticMeshRenderData* UStaticMesh::GetRenderData() const
{
	WaitUntilAsyncPropertyReleased(EStaticMeshAsyncProperties::RenderData);

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	return RenderData.Get();
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

void UStaticMesh::SetRenderData(TUniquePtr<class FStaticMeshRenderData>&& InRenderData)
{
	WaitUntilAsyncPropertyReleased(EStaticMeshAsyncProperties::RenderData);

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	RenderData = MoveTemp(InRenderData);
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

void UStaticMesh::PostInitProperties()
{
#if WITH_EDITORONLY_DATA
	if (!HasAnyFlags(RF_ClassDefaultObject))
	{
		SetAssetImportData(NewObject<UAssetImportData>(this, TEXT("AssetImportData")));
	}
#endif
	Super::PostInitProperties();
}

/**
 * Initializes the static mesh's render resources.
 */
void UStaticMesh::InitResources()
{
	LLM_SCOPE_BYNAME(TEXT("StaticMesh/InitResources")); // This is an important test case for SCOPE_BYNAME without a matching LLM_DEFINE_TAG

	bRenderingResourcesInitialized = true;

	UpdateUVChannelData(false);

	CachedSRRState.Clear();
	if (FStaticMeshRenderData* RenderDataLocal = GetRenderData())
	{
		{
			const int32 NumLODs = GetNumLODs();
			const int32 MinFirstLOD = GetMinLODIdx(true);

			CachedSRRState.NumNonStreamingLODs = RenderDataLocal->NumInlinedLODs;
			CachedSRRState.NumNonOptionalLODs = RenderDataLocal->GetNumNonOptionalLODs();
			// Limit the number of LODs based on MinLOD value.
			CachedSRRState.MaxNumLODs = FMath::Clamp<int32>(NumLODs - MinFirstLOD, RenderDataLocal->NumInlinedLODs, NumLODs);
			CachedSRRState.AssetLODBias = MinFirstLOD;
			CachedSRRState.LODBiasModifier = RenderDataLocal->LODBiasModifier;
			// The optional LOD might be culled now.
			CachedSRRState.NumNonOptionalLODs = FMath::Min(CachedSRRState.NumNonOptionalLODs, CachedSRRState.MaxNumLODs);
			// Set LOD count to fit the current state.
			CachedSRRState.NumResidentLODs = NumLODs - RenderDataLocal->CurrentFirstLODIdx;
			CachedSRRState.NumRequestedLODs = CachedSRRState.NumResidentLODs;
			// Set whether the mips can be streamed.
			CachedSRRState.bSupportsStreaming = !NeverStream && CachedSRRState.NumNonStreamingLODs != CachedSRRState.MaxNumLODs;
		}

		// TODO : Update RenderData->CurrentFirstLODIdx based on whether IStreamingManager::Get().IsRenderAssetStreamingEnabled(EStreamableRenderAssetType::StaticMesh).
		// TODO : This will require to refactor code in FStaticMeshLODResources::Serialize() and FStaticMeshRenderData::Cache() around bBuffersInlined (in cooked).

		UWorld* World = GetWorld();
		RenderDataLocal->InitResources(World ? World->GetFeatureLevel() : ERHIFeatureLevel::Num, this);
		CachedSRRState.bHasPendingInitHint = true;
	}

#if (WITH_EDITOR && DO_CHECK)
	if (GetRenderData() && CachedSRRState.bSupportsStreaming && !GetOutermost()->bIsCookedForEditor)
	{
		const int32 NumLODs = GetNumLODs();

		for (int32 LODIdx = 0; LODIdx < NumLODs; ++LODIdx)
		{
			const FStaticMeshLODResources& LODResource = GetRenderData()->LODResources[LODIdx];
			check(LODResource.bBuffersInlined || !LODResource.DerivedDataKey.IsEmpty());
		}
	}
#endif

	LinkStreaming();

#if	STATS
	// Compute size on the current thread to avoid the render thread causing a stall on accessing RenderData
	// before it is released from async duty
	const uint32 StaticMeshResourceSize = GetResourceSizeBytes(EResourceSizeMode::Exclusive);
	ENQUEUE_RENDER_COMMAND(UpdateMemoryStats)(
		[StaticMeshResourceSize](FRHICommandList& RHICmdList)
		{
			INC_DWORD_STAT_BY( STAT_StaticMeshTotalMemory, StaticMeshResourceSize );
			INC_DWORD_STAT_BY( STAT_StaticMeshTotalMemory2, StaticMeshResourceSize );
		} );
#endif // STATS
}

void UStaticMesh::GetResourceSizeEx(FResourceSizeEx& CumulativeResourceSize)
{
	Super::GetResourceSizeEx(CumulativeResourceSize);

	if (GetRenderData())
	{
		GetRenderData()->GetResourceSizeEx(CumulativeResourceSize);
	}
}

void FStaticMeshRenderData::GetResourceSizeEx(FResourceSizeEx& CumulativeResourceSize) const
{
	CumulativeResourceSize.AddDedicatedSystemMemoryBytes(sizeof(*this));

	// Count dynamic arrays.
	CumulativeResourceSize.AddUnknownMemoryBytes(LODResources.GetAllocatedSize());

	for(int32 LODIndex = 0;LODIndex < LODResources.Num();LODIndex++)
	{
		const FStaticMeshLODResources& LODRenderData = LODResources[LODIndex];

		LODResources[LODIndex].GetResourceSizeEx(CumulativeResourceSize);

		if (LODRenderData.DistanceFieldData)
		{
			LODRenderData.DistanceFieldData->GetResourceSizeEx(CumulativeResourceSize);
		}

		if (LODRenderData.CardRepresentationData)
		{
			LODRenderData.CardRepresentationData->GetResourceSizeEx(CumulativeResourceSize);
		}
	}

#if WITH_EDITORONLY_DATA
	// If render data for multiple platforms is loaded, count it all.
	if (NextCachedRenderData)
	{
		NextCachedRenderData->GetResourceSizeEx(CumulativeResourceSize);
	}
#endif // #if WITH_EDITORONLY_DATA

	GetNaniteResourcesSizeEx(NaniteResourcesPtr, CumulativeResourceSize);
}

SIZE_T FStaticMeshRenderData::GetCPUAccessMemoryOverhead() const
{
	SIZE_T Result = 0;

	for (int32 LODIndex = 0; LODIndex < LODResources.Num(); ++LODIndex)
	{
		Result += LODResources[LODIndex].GetCPUAccessMemoryOverhead();
	}

#if WITH_EDITORONLY_DATA
	Result += NextCachedRenderData ? NextCachedRenderData->GetCPUAccessMemoryOverhead() : 0;
#endif
	return Result;
}

int32 UStaticMesh::GetNumVertices(int32 LODIndex) const
{
	int32 NumVertices = 0;
	if (GetRenderData() && GetRenderData()->LODResources.IsValidIndex(LODIndex))
	{
		NumVertices = GetRenderData()->LODResources[LODIndex].GetNumVertices();
	}
	return NumVertices;
}

int32 UStaticMesh::GetNumTriangles(int32 LODIndex) const
{
	int32 NumTriangles = 0;
	if (GetRenderData() && GetRenderData()->LODResources.IsValidIndex(LODIndex))
	{
		NumTriangles = GetRenderData()->LODResources[LODIndex].GetNumTriangles();
	}
	return NumTriangles;
}

int32 UStaticMesh::GetNumTexCoords(int32 LODIndex) const
{
	int32 NumTexCoords = 0;
	if (GetRenderData() && GetRenderData()->LODResources.IsValidIndex(LODIndex))
	{
		NumTexCoords = GetRenderData()->LODResources[LODIndex].GetNumTexCoords();
	}
	return NumTexCoords;
}

int32 UStaticMesh::GetNumNaniteVertices() const
{
	int32 NumVertices = 0;
	if (HasValidNaniteData())
	{
		const Nanite::FResources& Resources = *GetRenderData()->NaniteResourcesPtr.Get();
		if (Resources.RootData.Num() > 0)
		{
			NumVertices = Resources.NumInputVertices;
		}
	}
	return NumVertices;
}

int32 UStaticMesh::GetNumNaniteTriangles() const
{
	int32 NumTriangles = 0;
	if (HasValidNaniteData())
	{
		const Nanite::FResources& Resources = *GetRenderData()->NaniteResourcesPtr.Get();
		if (Resources.RootData.Num() > 0)
		{
			NumTriangles = Resources.NumInputTriangles;
		}
	}
	return NumTriangles;
}

int32 UStaticMesh::GetNumLODs() const
{
	int32 NumLODs = 0;
	if (GetRenderData())
	{
		NumLODs = GetRenderData()->LODResources.Num();
	}
	return NumLODs;
}

// pass false for bCheckLODForVerts for any runtime code that can handle empty LODs, for example due to them being stripped
//  as a result of minimum LOD setup on the static mesh; in cooked builds, those verts are stripped, but systems still need to
//  be able to handle these cases; to check specifically for an LOD, pass true (default arg), and an LOD index (default arg implies MinLOD)
//
bool UStaticMesh::HasValidRenderData(bool bCheckLODForVerts, int32 LODIndex) const
{
	if (GetRenderData() && GetRenderData()->LODResources.Num() && GetRenderData()->LODResources.GetData())
	{
		if (bCheckLODForVerts)
		{
			if (LODIndex == INDEX_NONE)
			{
				LODIndex = FMath::Clamp<int32>(GetMinLODIdx(), 0, GetRenderData()->LODResources.Num() - 1);
			}

			return (GetRenderData()->LODResources[LODIndex].VertexBuffers.StaticMeshVertexBuffer.GetNumVertices() > 0);
		}
		else
		{
			return true;
		}
	}
	return false;
}

bool UStaticMesh::HasValidNaniteData() const
{
	if (const FStaticMeshRenderData* SMRenderData = GetRenderData())
	{
		return SMRenderData->HasValidNaniteData();
	}

	return false;
}

bool UStaticMesh::HasNaniteFallbackMesh(EShaderPlatform ShaderPlatform) const
{	
	if (const FStaticMeshRenderData* SMRenderData = GetRenderData())
	{
		return SMRenderData->HasNaniteFallbackMesh(ShaderPlatform);
	}

	return false;
}

FBoxSphereBounds UStaticMesh::GetBounds() const
{
	return GetExtendedBounds();
}

FBox UStaticMesh::GetBoundingBox() const
{
	return GetExtendedBounds().GetBox();
}

int32 UStaticMesh::GetNumSections(int32 InLOD) const
{
	int32 NumSections = 0;
	if (GetRenderData() && GetRenderData()->LODResources.IsValidIndex(InLOD))
	{
		const FStaticMeshLODResources& LOD = GetRenderData()->LODResources[InLOD];
		NumSections = LOD.Sections.Num();
	}
	return NumSections;
}

#if WITH_EDITORONLY_DATA
static float GetUVDensity(const FStaticMeshLODResourcesArray& LODResources, int32 UVIndex)
{
	float WeightedUVDensity = 0;
	float WeightSum = 0;

	if (UVIndex < MAX_STATIC_TEXCOORDS)
	{
		// Parse all LOD-SECTION using this material index.
		for (const FStaticMeshLODResources& LODModel : LODResources)
		{
			if (UVIndex < LODModel.GetNumTexCoords())
			{
				for (const FStaticMeshSection& SectionInfo : LODModel.Sections)
				{
					WeightedUVDensity += SectionInfo.UVDensities[UVIndex] * SectionInfo.Weights[UVIndex];
					WeightSum += SectionInfo.Weights[UVIndex];
				}
			}
		}
	}

	return (WeightSum > UE_SMALL_NUMBER) ? (WeightedUVDensity / WeightSum) : 0;
}
#endif

void UStaticMesh::UpdateUVChannelData(bool bRebuildAll)
{
#if WITH_EDITORONLY_DATA
	TRACE_CPUPROFILER_EVENT_SCOPE(UStaticMesh::UpdateUVChannelData);

	// Once cooked, the data required to compute the scales will not be CPU accessible.
	if (FPlatformProperties::HasEditorOnlyData() && GetRenderData())
	{
		bool bDensityChanged = false;

		for (int32 MaterialIndex = 0; MaterialIndex < GetStaticMaterials().Num(); ++MaterialIndex)
		{
			FMeshUVChannelInfo& UVChannelData = GetStaticMaterials()[MaterialIndex].UVChannelData;

			// Skip it if we want to keep it.
			if (UVChannelData.IsInitialized() && (!bRebuildAll || UVChannelData.bOverrideDensities))
			{
				continue;
			}

			float WeightedUVDensities[TEXSTREAM_MAX_NUM_UVCHANNELS] = {0, 0, 0, 0};
			float Weights[TEXSTREAM_MAX_NUM_UVCHANNELS] = {0, 0, 0, 0};

			// Parse all LOD-SECTION using this material index.
			for (const FStaticMeshLODResources& LODModel : GetRenderData()->LODResources)
			{
				const int32 NumTexCoords = FMath::Min<int32>(LODModel.GetNumTexCoords(), TEXSTREAM_MAX_NUM_UVCHANNELS);
				for (const FStaticMeshSection& SectionInfo : LODModel.Sections)
				{
					if (SectionInfo.MaterialIndex == MaterialIndex)
					{
						for (int32 UVIndex = 0; UVIndex < NumTexCoords; ++UVIndex)
						{
							WeightedUVDensities[UVIndex] += SectionInfo.UVDensities[UVIndex] * SectionInfo.Weights[UVIndex];
							Weights[UVIndex] += SectionInfo.Weights[UVIndex];
						}

						// If anything needs to be updated, also update the lightmap densities.
						bDensityChanged = true;
					}
				}
			}

			UVChannelData.bInitialized = true;
			UVChannelData.bOverrideDensities = false;
			for (int32 UVIndex = 0; UVIndex < TEXSTREAM_MAX_NUM_UVCHANNELS; ++UVIndex)
			{
				UVChannelData.LocalUVDensities[UVIndex] = (Weights[UVIndex] > UE_SMALL_NUMBER) ? (WeightedUVDensities[UVIndex] / Weights[UVIndex]) : 0;
			}
		}

		if (bDensityChanged || bRebuildAll)
		{
			SetLightmapUVDensity(GetUVDensity(GetRenderData()->LODResources, GetLightMapCoordinateIndex()));

			// This can potentially be run from any thread during async static mesh compilation
			if (GEngine)
			{
				if (IsInGameThread())
				{
					GEngine->TriggerStreamingDataRebuild();
				}
				else
				{
					// GEngine could be null by the time the task gets executed on the task graph.
					Async(EAsyncExecution::TaskGraphMainThread, []() { if (GEngine) { GEngine->TriggerStreamingDataRebuild(); } });
				}
			}
		}

		// Update the data for the renderthread debug viewmodes
		GetRenderData()->SyncUVChannelData(GetStaticMaterials());
	}
#endif
}

#if WITH_EDITORONLY_DATA
static void AccumulateBounds(FBox& Bounds, const FStaticMeshLODResources& LODModel, const FStaticMeshSection& SectionInfo, const FTransform& Transform)
{
	const int32 FirstIndex = SectionInfo.FirstIndex;
	const int32 LastIndex = FirstIndex + SectionInfo.NumTriangles * 3;
	const int32 NumIndices = LODModel.IndexBuffer.GetNumIndices();

	if (LastIndex < NumIndices)
	{
		const FIndexArrayView IndexBuffer = LODModel.IndexBuffer.GetArrayView();
		for (int32 Index = FirstIndex; Index < LastIndex; ++Index)
		{
			Bounds += Transform.TransformPosition(FVector(LODModel.VertexBuffers.PositionVertexBuffer.VertexPosition(IndexBuffer[Index])));
		}
	}
}
#endif

FBox UStaticMesh::GetMaterialBox(int32 MaterialIndex, const FTransform& Transform) const
{
	FBox MaterialBounds(ForceInit);

#if WITH_EDITORONLY_DATA
	// Once cooked, the data requires to compute the scales will not be CPU accessible.
	if (FPlatformProperties::HasEditorOnlyData() && GetRenderData())
	{
		for (const FStaticMeshLODResources& LODModel : GetRenderData()->LODResources)
		{
			for (const FStaticMeshSection& SectionInfo : LODModel.Sections)
			{
				if (SectionInfo.MaterialIndex != MaterialIndex)
					continue;

				AccumulateBounds(MaterialBounds, LODModel, SectionInfo, Transform);
			}
		}
	}
#endif

	if (!MaterialBounds.IsValid)
	{
		// Fallback back using the full bounds.
		MaterialBounds = GetBoundingBox().TransformBy(Transform);
	}

	return MaterialBounds;
}

const FMeshUVChannelInfo* UStaticMesh::GetUVChannelData(int32 MaterialIndex) const
{
	if (GetStaticMaterials().IsValidIndex(MaterialIndex))
	{
		ensure(GetStaticMaterials()[MaterialIndex].UVChannelData.bInitialized);
		return &GetStaticMaterials()[MaterialIndex].UVChannelData;
	}

	return nullptr;
}

/**
 * Releases the static mesh's render resources.
 */
void UStaticMesh::ReleaseResources()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UStaticMesh::ReleaseResources);
#if STATS
	uint32 StaticMeshResourceSize = GetResourceSizeBytes(EResourceSizeMode::Exclusive);
	DEC_DWORD_STAT_BY( STAT_StaticMeshTotalMemory, StaticMeshResourceSize );
	DEC_DWORD_STAT_BY( STAT_StaticMeshTotalMemory2, StaticMeshResourceSize );
#endif

	if (GetRenderData())
	{
		GetRenderData()->ReleaseResources();

		// insert a fence to signal when these commands completed
		ReleaseResourcesFence.BeginFence();
	}

	bRenderingResourcesInitialized = false;
}

#if WITH_EDITOR
void UStaticMesh::PreEditChange(FProperty* PropertyAboutToChange)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UStaticMesh::PreEditChange);

	if (bIsInPostEditChange)
	{
		//Ignore re-entrant PostEditChange calls
		return;
	}

	// Stall for compilation of this mesh and any other mesh that might be reading from it concurrently
	FStaticMeshCompilingManager::FFinishCompilationOptions FinishCompilationOptions;
	FinishCompilationOptions.bIncludeDependentMeshes = true;
	FStaticMeshCompilingManager::Get().FinishCompilation({ this }, FinishCompilationOptions);

	// We need to cancel these builds manually since they rely on PostGC reachability analysis to delete invalid tasks themselves.
	// If the mesh is invalidated and the async task executes before GC we may attempt to build with invalid data.
	// PostEditChange will enqueue builds for each of these again so it's okay to cancel them completely here.
	// #todo: this should be modified to use the FAssetCompilationManager::Get().FinishAllCompilationForObjects function in 5.2
	{
		if (GDistanceFieldAsyncQueue)
		{
			GDistanceFieldAsyncQueue->CancelBuild(this);
		}
		if (GCardRepresentationAsyncQueue)
		{
			GCardRepresentationAsyncQueue->CancelBuild(this);
		}
	}


	Super::PreEditChange(PropertyAboutToChange);

	// Release the static mesh's resources.
	ReleaseResources();

	// Invalidate the render data for any components using this static mesh. This is essentially the same work done by the
	// FStaticMeshComponentRecreateRenderStateContext constructor, but we don't want to re-create the render state immediately.
	TSet<FSceneInterface*> Scenes;
	FObjectCacheContextScope ObjectCacheScope;
	for (IStaticMeshComponent* Component : ObjectCacheScope.GetContext().GetStaticMeshComponents(this))
	{
		IPrimitiveComponent* PrimComponent = Component->GetPrimitiveComponentInterface();

		if (PrimComponent->IsRenderStateCreated())
		{
			PrimComponent->DestroyRenderState();
			Scenes.Add(PrimComponent->GetScene());
		}
	}

	UpdateAllPrimitiveSceneInfosForScenes(MoveTemp(Scenes));

	// Flush the resource release commands to the rendering thread to ensure that the edit change doesn't occur while a resource is still
	// allocated, and potentially accessing the UStaticMesh.
	ReleaseResourcesFence.Wait();
}

void UStaticMesh::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UStaticMesh::PostEditChangeProperty);

	FProperty* PropertyThatChanged = PropertyChangedEvent.Property;
	const FName PropertyName = PropertyThatChanged ? PropertyThatChanged->GetFName() : NAME_None;
	
	if (bIsInPostEditChange)
	{
		//Ignore re-entrant PostEditChange calls
		return;
	}
	TGuardValue<bool> PostEditChangeGuard(bIsInPostEditChange, true);

	if (PropertyName == UStaticMesh::GetLODGroupMemberName())
	{
		// Force an update of LOD group settings

		// Don't rebuild inside here.  We're doing that below.
		constexpr bool bRebuild = false;
		SetLODGroup(GetLODGroup(), bRebuild);
	}
#if WITH_EDITORONLY_DATA
	if (PropertyName == GET_MEMBER_NAME_CHECKED(UStaticMesh, ComplexCollisionMesh) && ComplexCollisionMesh != this)
	{
		if (GetBodySetup())
		{
			GetBodySetup()->InvalidatePhysicsData();
			GetBodySetup()->CreatePhysicsMeshes();
		}
	}

	if (PropertyName == GET_MEMBER_NAME_CHECKED(UStaticMesh, bSupportPhysicalMaterialMasks))
	{
		if (GetBodySetup())
		{
			GetBodySetup()->bSupportUVsAndFaceRemap = bSupportPhysicalMaterialMasks;
			GetBodySetup()->InvalidatePhysicsData();
			GetBodySetup()->CreatePhysicsMeshes();
		}
	}
#endif

	if (PropertyName == GetLightMapResolutionName())
	{
		SetLightMapResolution(FMath::Max(GetLightMapResolution(), 0));
	}

	if (PropertyChangedEvent.MemberProperty 
		&& (PropertyChangedEvent.MemberProperty->GetFName() == UStaticMesh::GetPositiveBoundsExtensionName() || 
			PropertyChangedEvent.MemberProperty->GetFName() == UStaticMesh::GetNegativeBoundsExtensionName()))
	{
		// Update the extended bounds
		CalculateExtendedBounds();
	}

	if (!GetAutoComputeLODScreenSize()
		&& GetRenderData()
		&& PropertyName == UStaticMesh::GetAutoComputeLODScreenSizeMemberName())
	{
		for (int32 LODIndex = 1; LODIndex < GetNumSourceModels(); ++LODIndex)
		{
			GetSourceModel(LODIndex).ScreenSize = GetRenderData()->ScreenSize[LODIndex];
		}
	}

	if (PropertyChangedEvent.ChangeType != EPropertyChangeType::Interactive)
	{
		// Only unbuild lighting for properties which affect static lighting
		if (PropertyName == UStaticMesh::GetLightMapResolutionName()
			|| PropertyName == UStaticMesh::GetLightMapCoordinateIndexName())
		{
			SetLightingGuid();
		}
	}

	if (PropertyName == UStaticMesh::GetNaniteSettingsMemberName())
	{
		CheckForMissingShaderModels();

		if (GetNaniteSettings().bEnabled)
		{
			// If Nanite was enabled we can no longer do fast build
			bDoFastBuild = false;
		}
	}

	UStaticMesh::FBuildParameters BuildParameters;
	BuildParameters.bInSilent = true;
	BuildParameters.bInRebuildUVChannelData = true;
	BuildParameters.bInEnforceLightmapRestrictions = true;
	Build(BuildParameters);

	if (PropertyName == GET_MEMBER_NAME_CHECKED(UStaticMesh, bHasNavigationData)
		|| PropertyName == UStaticMesh::GetBodySetupName())
	{
		// Build called above will result in creation, update or destruction 
		// of NavCollision. We need to let related StaticMeshComponents know
		BroadcastNavCollisionChange();
	}
	
	for (UAssetUserData* Datum : AssetUserData)
	{
		if (Datum != nullptr)
		{
			Datum->PostEditChangeOwner(PropertyChangedEvent);
		}
	}

	Super::PostEditChangeProperty(PropertyChangedEvent);

	OnMeshChanged.Broadcast();
}

void UStaticMesh::PostEditUndo()
{
	// The super will cause a Build() via PostEditChangeProperty().
	Super::PostEditUndo();
}


#if WITH_EDITOR
EDataValidationResult UStaticMesh::IsDataValid(FDataValidationContext& Context) const
{
	EDataValidationResult ValidationResult = Super::IsDataValid(Context);

	// a cooked static mesh asset is probably not going to have valid SourceModels (?)
	if (GetPackage()->HasAnyPackageFlags(PKG_Cooked) == false)
	{
		if (GetSourceModels().Num() == 0)
		{
			Context.AddError(LOCTEXT("StaticMeshValidation_NoSourceModel", "This Static Mesh Asset has no Source Models. This asset is not repairable, the Asset is corrupted and must be deleted."));
			ValidationResult = EDataValidationResult::Invalid;
		}
		else if (GetSourceModels()[0].IsSourceModelInitialized() == false)
		{
			Context.AddError(LOCTEXT("StaticMeshValidation_UninitializedLOD0", "This Static Mesh Asset has no LOD0 Source Model mesh. This asset is not repairable, the Asset is corrupted and must be deleted."));
			ValidationResult = EDataValidationResult::Invalid;
		}
		
		if (!GIsBuildMachine)
		{
			const int32 LodCount = GetNumSourceModels();
			for (int32 LodIndex = 0; LodIndex < LodCount; ++LodIndex)
			{
				const FStaticMeshSourceModel& SrcModel = GetSourceModel(LodIndex);
				if (const FMeshDescription* LodMeshDescription = GetMeshDescription(LodIndex))
				{
					if (LodMeshDescription->IsEmpty())
					{
						FFormatNamedArguments Args;
						Args.Add(TEXT("LODIndex"), LodIndex);
						Context.AddError(FText::Format(LOCTEXT("StaticMeshValidation_EmptyLodMeshDescription", "Empty lod {LODIndex} mesh geometry. Lod {LODIndex} mesh geometry should not be empty."), Args));
						ValidationResult = EDataValidationResult::Invalid;
					}
					//If we have the lod 0 meshdescription, verify the hi-res mesh description
					if (LodIndex == 0 && IsHiResMeshDescriptionValid())
					{
						if (const FMeshDescription* HiResMeshDescription = GetHiResMeshDescription())
						{
							if (HiResMeshDescription->IsEmpty())
							{
								Context.AddError(LOCTEXT("StaticMeshValidation_EmptyHiResMeshDescription", "Empty hi-res mesh geometry. Nanite hi-res mesh geometry should not be empty."));
								ValidationResult = EDataValidationResult::Invalid;
							}

							//Validate the number of sections
							if (HiResMeshDescription->PolygonGroups().Num() > LodMeshDescription->PolygonGroups().Num())
							{
								Context.AddError(LOCTEXT("StaticMeshValidation_HiresMoreSectionThanLod0", "Invalid hi-res mesh description. The number of sections from the hires mesh is higher than LOD 0 section count. This is not supported and LOD 0 will be used as a fallback to build nanite data."));
								ValidationResult = EDataValidationResult::Invalid;
							}
						}
					}
				}
			}
		}
	}

	{
		// check the MinLOD values are all within range
		FPerQualityLevelInt QualityLocalMinLOD;
		FPerPlatformInt LocalMinLOD;
		int32 MinAvailableLOD = INDEX_NONE;
		TArray<TPair<int32, FName>> InvalidMinLODs;
		CheckForValidMinLODs(QualityLocalMinLOD, LocalMinLOD, MinAvailableLOD, InvalidMinLODs);
		if (InvalidMinLODs.Num() > 0)
		{
			for (const TPair<int32, FName>& InvalidMinLOD : InvalidMinLODs)
			{
				const int32 LODIdx = InvalidMinLOD.Key;
				const FName OverrideName = InvalidMinLOD.Value;

				FFormatNamedArguments Arguments;
				Arguments.Add(TEXT("MinLOD"), FText::AsNumber(LODIdx));
				Arguments.Add(TEXT("MinAvailLOD"), FText::AsNumber(MinAvailableLOD));
				Arguments.Add(TEXT("OverrideName"), FText::FromName(OverrideName));

				if (OverrideName.IsNone())
				{
					Context.AddWarning(FText::Format(LOCTEXT("LoadError_BadMinLOD", "Min LOD value of {MinLOD} is out of range 0..{MinAvailLOD}."), Arguments));
				}
				else
				{
					Context.AddWarning(FText::Format(LOCTEXT("LoadError_BadMinLODWithOverride", "Min LOD override of {MinLOD} for {OverrideName} is out of range 0..{MinAvailLOD}."), Arguments));
				}
			}

			ValidationResult = EDataValidationResult::Invalid;
		}
	}

	return ValidationResult;
}
#endif


void UStaticMesh::SetLODGroup(FName NewGroup, bool bRebuildImmediately, bool bAllowModify)
{
#if WITH_EDITORONLY_DATA
	const bool bBeforeDerivedDataCached = (GetRenderData() == nullptr);
	if (!bBeforeDerivedDataCached && bAllowModify)
	{
		Modify();
	}
	bool bResetSectionInfoMap = (GetLODGroup() != NewGroup);
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	LODGroup = NewGroup;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	if (NewGroup != NAME_None)
	{
		const ITargetPlatform* Platform = GetTargetPlatformManagerRef().GetRunningTargetPlatform();
		check(Platform);
		const FStaticMeshLODGroup& GroupSettings = Platform->GetStaticMeshLODSettings().GetLODGroup(NewGroup);

		// Set the number of LODs to at least the default. If there are already LODs they will be preserved, with default settings of the new LOD group.
		int32 DefaultLODCount = GroupSettings.GetDefaultNumLODs();

		SetNumSourceModels(DefaultLODCount);
		
		for (int32 LODIndex = 0; LODIndex < DefaultLODCount; ++LODIndex)
		{
			FStaticMeshSourceModel& SourceModel = GetSourceModel(LODIndex);

			// Set reduction settings to the defaults.
			SourceModel.ReductionSettings = GroupSettings.GetDefaultSettings(LODIndex);
			
			const bool bIsReductionActive = IsReductionActive(LODIndex);
			if (LODIndex != 0 && bIsReductionActive)
			{
				//Reset the section info map
				if (bResetSectionInfoMap)
				{
					const int32 NumberOfSections = GetSectionInfoMap().GetSectionNumber(LODIndex);
					for (int32 SectionIndex = 0; SectionIndex < NumberOfSections; ++SectionIndex)
					{
						GetSectionInfoMap().Remove(LODIndex, SectionIndex);
					}
				}
				//Clear the raw data if we change the LOD Group and we do not reduce ourself, this will force the user to do a import LOD which will manage the section info map properly
				if (!SourceModel.IsRawMeshEmpty() && SourceModel.ReductionSettings.BaseLODModel != LODIndex)
				{
					FRawMesh EmptyRawMesh;
					SourceModel.SaveRawMesh(EmptyRawMesh);
					SourceModel.SourceImportFilename = FString();
				}
			}
		}
		SetLightMapResolution(GroupSettings.GetDefaultLightMapResolution());

		if (!bBeforeDerivedDataCached)
		{
			SetAutoComputeLODScreenSize(true);
		}
	}
	if (bRebuildImmediately && !bBeforeDerivedDataCached)
	{
		PostEditChange();
	}
#endif
}

void UStaticMesh::BroadcastNavCollisionChange()
{
	if (FNavigationSystem::WantsComponentChangeNotifies())
	{
		for (FThreadSafeObjectIterator Iter(UStaticMeshComponent::StaticClass()); Iter; ++Iter)
		{
			UStaticMeshComponent* StaticMeshComponent = Cast<UStaticMeshComponent>(*Iter);
			UWorld* MyWorld = StaticMeshComponent->GetWorld();
			if (StaticMeshComponent->GetStaticMesh() == this)
			{
				StaticMeshComponent->bNavigationRelevant = StaticMeshComponent->IsNavigationRelevant();
				FNavigationSystem::UpdateComponentData(*StaticMeshComponent);
			}
		}
	}
}

FMeshSectionInfoMap& UStaticMesh::GetSectionInfoMap()
{
	WaitUntilAsyncPropertyReleased(EStaticMeshAsyncProperties::SectionInfoMap);
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	return SectionInfoMap; 
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

const FMeshSectionInfoMap& UStaticMesh::GetSectionInfoMap() const
{
	WaitUntilAsyncPropertyReleased(EStaticMeshAsyncProperties::SectionInfoMap);
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	return SectionInfoMap; 
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

FMeshSectionInfoMap& UStaticMesh::GetOriginalSectionInfoMap()
{
	WaitUntilAsyncPropertyReleased(EStaticMeshAsyncProperties::OriginalSectionInfoMap);
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	return OriginalSectionInfoMap; 
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

const FMeshSectionInfoMap& UStaticMesh::GetOriginalSectionInfoMap() const
{ 
	WaitUntilAsyncPropertyReleased(EStaticMeshAsyncProperties::OriginalSectionInfoMap);
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	return OriginalSectionInfoMap; 
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

int32 UStaticMesh::GetNumSourceModels() const
{
	return GetSourceModels().Num();
}

bool UStaticMesh::IsSourceModelValid(int32 Index) const
{
	return GetSourceModels().IsValidIndex(Index);
}

const TArray<FStaticMeshSourceModel>& UStaticMesh::GetSourceModels() const
{
	WaitUntilAsyncPropertyReleased(EStaticMeshAsyncProperties::SourceModels);
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	return SourceModels;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

const FStaticMeshSourceModel& UStaticMesh::GetSourceModel(int32 Index) const
{
	return GetSourceModels()[Index];
}

FStaticMeshSourceModel& UStaticMesh::GetSourceModel(int32 Index)
{ 
	WaitUntilAsyncPropertyReleased(EStaticMeshAsyncProperties::SourceModels);
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	return SourceModels[Index];
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

bool UStaticMesh::SetCustomLOD(const UStaticMesh* SourceStaticMesh, int32 DestinationLodIndex, const FString& SourceDataFilename)
{
#if WITH_EDITORONLY_DATA
	if (!ensure(SourceStaticMesh) || SourceStaticMesh->GetNumSourceModels() <= 0)
	{
		return false;
	}

	const int32 SourceLodIndex = (SourceStaticMesh->GetNumSourceModels() > DestinationLodIndex) ? DestinationLodIndex : 0;
	if(!SourceStaticMesh->IsSourceModelValid(SourceLodIndex))
	{
		return false;
	}

	const FMeshDescription* SourceMeshDescription = SourceStaticMesh->GetMeshDescription(SourceLodIndex);
	if(!SourceMeshDescription)
	{
		return false;
	}

	const bool bIsReimport = GetNumSourceModels() > DestinationLodIndex;

	if(DestinationLodIndex >= GetNumSourceModels())
	{
		// Add one LOD 
		AddSourceModel();
		if (GetNumSourceModels() <= DestinationLodIndex)
		{
			DestinationLodIndex = GetNumSourceModels() - 1;
		}
	}

	//To restore the material and section data, we need to know which material the source is using
	const TArray<FStaticMaterial>& SourceMaterials = SourceStaticMesh->GetStaticMaterials();
	TMap<int32, FName> SourceImportedMaterialNameUsed;
	{
		FStaticMeshConstAttributes SourceAttributes(*SourceMeshDescription);
		TPolygonGroupAttributesConstRef<FName> MaterialSlotNames = SourceAttributes.GetPolygonGroupMaterialSlotNames();
		for (FPolygonGroupID PolygonGroupID : SourceMeshDescription->PolygonGroups().GetElementIDs())
		{
			const int32 SourceMaterialIndex = PolygonGroupID.GetValue();
			check(SourceMaterials.IsValidIndex(SourceMaterialIndex));
			SourceImportedMaterialNameUsed.FindOrAdd(SourceMaterialIndex) = MaterialSlotNames[PolygonGroupID];
		}
	}

	TArray<FStaticMaterial>& DestinationMaterials = GetStaticMaterials();
	TMap<FName, FMeshSectionInfo> ExistingSectionInfos;
	FMeshDescription* DestinationMeshDescription = GetMeshDescription(DestinationLodIndex);
	if (DestinationMeshDescription == nullptr)
	{
		DestinationMeshDescription = CreateMeshDescription(DestinationLodIndex);
		check(DestinationMeshDescription != nullptr);
		CommitMeshDescription(DestinationLodIndex);

		//Make sure an imported mesh do not get reduce if there was no mesh data before reimport.
		//In this case we have a generated LOD convert to a custom LOD
		FStaticMeshSourceModel& SrcModel = GetSourceModel(DestinationLodIndex);
		SrcModel.ResetReductionSetting();
	}
	else
	{
		ensure(bIsReimport);
		FStaticMeshConstAttributes ExistingAttributes(*DestinationMeshDescription);
		TPolygonGroupAttributesConstRef<FName> ExistingMaterialSlotNames = ExistingAttributes.GetPolygonGroupMaterialSlotNames();
		int32 SectionIndex = 0;
		for (FPolygonGroupID PolygonGroupID : DestinationMeshDescription->PolygonGroups().GetElementIDs())
		{
			FMeshSectionInfo& ExistingInfo = ExistingSectionInfos.FindOrAdd(ExistingMaterialSlotNames[PolygonGroupID]);
			int32 MaterialSlotIndex = GetMaterialIndexFromImportedMaterialSlotName(ExistingMaterialSlotNames[PolygonGroupID]);
			if (GetSectionInfoMap().IsValidSection(DestinationLodIndex, SectionIndex))
			{
				ExistingInfo = GetSectionInfoMap().Get(DestinationLodIndex, SectionIndex);
			}
			else
			{
				ExistingInfo.MaterialIndex = MaterialSlotIndex;
			}
			if (!DestinationMaterials.IsValidIndex(ExistingInfo.MaterialIndex))
			{
				//There was an invalid material index in the existing mesh section info
				//Assign the raw material slot that match with the name if valid, otherwise set it at 0
				ExistingInfo.MaterialIndex = MaterialSlotIndex != INDEX_NONE ? MaterialSlotIndex : 0;
			}
			SectionIndex++;
		}
		// clear out the old mesh data
		DestinationMeshDescription->Empty();
	}

	//Make sure all materials use by the new LOD is pointing on a valid static material
	int32 SectionIndex = 0;
	for(const TPair<int32, FName>& SourceImportedMaterialNamePair : SourceImportedMaterialNameUsed)
	{
		FName NameSearch = SourceImportedMaterialNamePair.Value;
		int32 MaterialSlotIndex = GetMaterialIndexFromImportedMaterialSlotName(NameSearch);
		if (!ExistingSectionInfos.Contains(NameSearch))
		{
			if (MaterialSlotIndex == INDEX_NONE)
			{
				//Add the missing material slot
				MaterialSlotIndex = DestinationMaterials.Add(FStaticMaterial(UMaterial::GetDefaultMaterial(MD_Surface), NameSearch, NameSearch));
			}
			FMeshSectionInfo NewInfo;
			NewInfo.MaterialIndex = MaterialSlotIndex;
			ExistingSectionInfos.Add(NameSearch, NewInfo);
		}
#if WITH_EDITOR
		FMeshSectionInfo Info = ExistingSectionInfos.FindChecked(NameSearch);
		GetSectionInfoMap().Remove(DestinationLodIndex, SectionIndex);
		GetSectionInfoMap().Set(DestinationLodIndex, SectionIndex, Info);
#endif //WITH_EDITOR
		SectionIndex++;
	}

	//Copy the mesh description of the source into the destination
	*DestinationMeshDescription = *SourceMeshDescription;
	
	UStaticMesh* ThisMesh = this;
	auto FinalizeSetCustomLODGameThread = [ThisMesh, DestinationLodIndex]()
	{
		check(IsInGameThread());
		//Commit the mesh description to update the ddc key
		FCommitMeshDescriptionParams CommitMeshDescriptionParams;
		CommitMeshDescriptionParams.bMarkPackageDirty = true;
		CommitMeshDescriptionParams.bUseHashAsGuid = false;
		ThisMesh->CommitMeshDescription(DestinationLodIndex, CommitMeshDescriptionParams);
		ThisMesh->PostEditChange();
	};

	if(IsInGameThread())
	{
		FinalizeSetCustomLODGameThread();
	}
	else
	{
		Async(EAsyncExecution::TaskGraphMainThread, MoveTemp(FinalizeSetCustomLODGameThread));
	}

	if (IsSourceModelValid(DestinationLodIndex))
	{
		FStaticMeshSourceModel& SourceModel = GetSourceModel(DestinationLodIndex);
		SourceModel.SourceImportFilename = UAssetImportData::SanitizeImportFilename(SourceDataFilename, nullptr);
		SourceModel.bImportWithBaseMesh = false;
	}

	RemoveUnusedMaterialSlots(this);

	return true;
#else
	return false;
#endif //!WITH_EDITORONLY_DATA
}

//Static function
void UStaticMesh::RemoveUnusedMaterialSlots(UStaticMesh* StaticMesh)
{
#if WITH_EDITOR
	if (!StaticMesh)
	{
		return;
	}

	TArray<FStaticMaterial>& Materials = StaticMesh->GetStaticMaterials();
	if (Materials.Num() < 2)
	{
		return;
	}

	FMeshSectionInfoMap& SectionInfoMap = StaticMesh->GetSectionInfoMap();
	const int32 LodCount = StaticMesh->GetNumSourceModels();

	//Clean up the material list by removing unused materials
	for (int32 MaterialIndex = Materials.Num() - 1; MaterialIndex >= 0; MaterialIndex--)
	{
		bool bMaterialIsUse = false;
		for (int32 LodIndex = 0; LodIndex < LodCount && !bMaterialIsUse; ++LodIndex)
		{
			const int32 SectionCount = SectionInfoMap.GetSectionNumber(LodIndex);
			for (int32 SectionIndex = 0; SectionIndex < SectionCount; ++SectionIndex)
			{
				FMeshSectionInfo SectionInfo = SectionInfoMap.Get(LodIndex, SectionIndex);
				if (SectionInfo.MaterialIndex == MaterialIndex)
				{
					bMaterialIsUse = true;
					break;
				}
			}
		}
		if (!bMaterialIsUse)
		{
			Materials.RemoveAt(MaterialIndex);
		}
		else
		{
			//Stop removing unused material when we find a valid one, to avoid patching any data related to material index.
			break;
		}
	}
#endif // WITH_EDITOR
}

FStaticMeshSourceModel& UStaticMesh::AddSourceModel()
{
	WaitUntilAsyncPropertyReleased(EStaticMeshAsyncProperties::SourceModels);
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	int32 LodModelIndex = SourceModels.AddDefaulted();
	FStaticMeshSourceModel& NewSourceModel = SourceModels[LodModelIndex];
	NewSourceModel.CreateSubObjects(this);
	return NewSourceModel;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

void UStaticMesh::SetNumSourceModels(const int32 Num)
{
	WaitUntilAsyncPropertyReleased(EStaticMeshAsyncProperties::SourceModels);
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	const int32 OldNum = SourceModels.Num();

	//Shrink the SectionInfoMap if some SourceModel are removed
	if (OldNum > Num)
	{
		for (int32 RemoveLODIndex = Num; RemoveLODIndex < OldNum; ++RemoveLODIndex)
		{
			// Remove MeshDescription allocations
			FStaticMeshSourceModel& ThisSourceModel = SourceModels[RemoveLODIndex];
			ThisSourceModel.ClearMeshDescription();
			check(ThisSourceModel.GetMeshDescriptionBulkData());
			ThisSourceModel.GetMeshDescriptionBulkData()->Empty();

			int32 SectionCount = GetSectionInfoMap().GetSectionNumber(RemoveLODIndex);
			for (int32 SectionIndex = 0; SectionIndex < SectionCount; ++SectionIndex)
			{
				GetSectionInfoMap().Remove(RemoveLODIndex, SectionIndex);
			}
			SectionCount = GetOriginalSectionInfoMap().GetSectionNumber(RemoveLODIndex);
			for (int32 SectionIndex = 0; SectionIndex < SectionCount; ++SectionIndex)
			{
				GetOriginalSectionInfoMap().Remove(RemoveLODIndex, SectionIndex);
			}
		}
	}

	SourceModels.SetNum(Num);

	for (int32 Index = OldNum; Index < Num; ++Index)
	{
		FStaticMeshSourceModel& ThisSourceModel = SourceModels[Index];

		ThisSourceModel.CreateSubObjects(this);
		int32 PreviousCustomLODIndex = 0;
		//Find the previous custom LOD
		for (int32 ReverseIndex = Index - 1; ReverseIndex > 0; ReverseIndex--)
		{
			const FStaticMeshSourceModel& StaticMeshModel = SourceModels[ReverseIndex];
			//If the custom import LOD is reduced and is not using itself as the source, do not consider it
			if (IsMeshDescriptionValid(ReverseIndex) && !(IsReductionActive(ReverseIndex) && StaticMeshModel.ReductionSettings.BaseLODModel != ReverseIndex))
			{
				PreviousCustomLODIndex = ReverseIndex;
				break;
			}
		}
		ThisSourceModel.ReductionSettings.BaseLODModel = PreviousCustomLODIndex;
		if (!IsMeshDescriptionValid(Index) && !IsReductionActive(Index))
		{
			//Set the Reduction percent
			ThisSourceModel.ReductionSettings.PercentTriangles = FMath::Pow(0.5f, (float)(Index-PreviousCustomLODIndex));
		}
	}
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

void UStaticMesh::RemoveSourceModel(const int32 Index)
{
	WaitUntilAsyncPropertyReleased(EStaticMeshAsyncProperties::SourceModels);
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	check(SourceModels.IsValidIndex(Index));

	// Remove MeshDescription allocations
	FStaticMeshSourceModel& ThisSourceModel = SourceModels[Index];
	ThisSourceModel.ClearMeshDescription();
	check(ThisSourceModel.GetMeshDescriptionBulkData());
	ThisSourceModel.GetMeshDescriptionBulkData()->Empty();

	//Remove the SectionInfoMap of the LOD we remove
	{
		int32 SectionCount = GetSectionInfoMap().GetSectionNumber(Index);
		for (int32 SectionIndex = 0; SectionIndex < SectionCount; ++SectionIndex)
		{
			GetSectionInfoMap().Remove(Index, SectionIndex);
		}
		SectionCount = GetOriginalSectionInfoMap().GetSectionNumber(Index);
		for (int32 SectionIndex = 0; SectionIndex < SectionCount; ++SectionIndex)
		{
			GetOriginalSectionInfoMap().Remove(Index, SectionIndex);
		}
	}

	//Move down all SectionInfoMap for the next LOD
	if (Index < SourceModels.Num() - 1)
	{
		for (int32 MoveIndex = Index + 1; MoveIndex < SourceModels.Num(); ++MoveIndex)
		{
			int32 SectionCount = GetSectionInfoMap().GetSectionNumber(MoveIndex);
			for (int32 SectionIndex = 0; SectionIndex < SectionCount; ++SectionIndex)
			{
				FMeshSectionInfo SectionInfo = GetSectionInfoMap().Get(MoveIndex, SectionIndex);
				GetSectionInfoMap().Set(MoveIndex - 1, SectionIndex, SectionInfo);
				GetSectionInfoMap().Remove(MoveIndex, SectionIndex);
			}
			SectionCount = GetOriginalSectionInfoMap().GetSectionNumber(MoveIndex);
			for (int32 SectionIndex = 0; SectionIndex < SectionCount; ++SectionIndex)
			{
				FMeshSectionInfo SectionInfo = GetOriginalSectionInfoMap().Get(MoveIndex, SectionIndex);
				GetOriginalSectionInfoMap().Set(MoveIndex - 1, SectionIndex, SectionInfo);
				GetOriginalSectionInfoMap().Remove(MoveIndex, SectionIndex);
			}
		}
	}

	//Remove the LOD
	SourceModels.RemoveAt(Index);
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

TArray<FStaticMeshSourceModel>&& UStaticMesh::MoveSourceModels()
{
	WaitUntilAsyncPropertyReleased(EStaticMeshAsyncProperties::SourceModels);
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	return MoveTemp(SourceModels);
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

void UStaticMesh::SetSourceModels(TArray<FStaticMeshSourceModel>&& InSourceModels)
{
	WaitUntilAsyncPropertyReleased(EStaticMeshAsyncProperties::SourceModels);
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	SourceModels = MoveTemp(InSourceModels);

	for (FStaticMeshSourceModel& SourceModel : SourceModels)
	{
		SourceModel.StaticMeshOwner = this;
		if (SourceModel.StaticMeshDescriptionBulkData)
		{
			SourceModel.StaticMeshDescriptionBulkData->Rename(nullptr, this, REN_DontCreateRedirectors | REN_NonTransactional);
		}
	}
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}


const FStaticMeshSourceModel& UStaticMesh::GetHiResSourceModel() const
{
	WaitUntilAsyncPropertyReleased(EStaticMeshAsyncProperties::HiResSourceModel);
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	return HiResSourceModel;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

FStaticMeshSourceModel& UStaticMesh::GetHiResSourceModel()
{
	WaitUntilAsyncPropertyReleased(EStaticMeshAsyncProperties::HiResSourceModel);
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	return HiResSourceModel;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

FStaticMeshSourceModel&& UStaticMesh::MoveHiResSourceModel()
{
	WaitUntilAsyncPropertyReleased(EStaticMeshAsyncProperties::HiResSourceModel);
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	return MoveTemp(HiResSourceModel);
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

void UStaticMesh::SetHiResSourceModel(FStaticMeshSourceModel&& InSourceModel)
{
	WaitUntilAsyncPropertyReleased(EStaticMeshAsyncProperties::HiResSourceModel);
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	HiResSourceModel = MoveTemp(InSourceModel);
	HiResSourceModel.StaticMeshOwner = this;
	if (HiResSourceModel.StaticMeshDescriptionBulkData)
	{
		HiResSourceModel.StaticMeshDescriptionBulkData->Rename(nullptr, this, REN_DontCreateRedirectors | REN_NonTransactional);
	}
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

bool UStaticMesh::TryCancelAsyncTasks()
{
	if (AsyncTask)
	{
		if (AsyncTask->IsIdle() || AsyncTask->IsDone() || AsyncTask->Cancel())
		{
			AsyncTask.Reset();
			ReleaseAsyncProperty();
		}
	}

	return AsyncTask == nullptr;
}

#endif // WITH_EDITOR

void UStaticMesh::BeginDestroy()
{
#if WITH_EDITORONLY_DATA
	ClearCachedNaniteAssemblyReferences();
#endif

	Super::BeginDestroy();

	if (!HasPendingInitOrStreaming() && bRenderingResourcesInitialized)
	{
		ReleaseResources();
	}
}

bool UStaticMesh::IsReadyForFinishDestroy()
{
#if WITH_EDITOR
	// We're being garbage collected and might still have async tasks pending
	if (!TryCancelAsyncTasks())
	{
		return false;
	}

	if (GetRenderData())
	{
		if (GDistanceFieldAsyncQueue)
		{
			GDistanceFieldAsyncQueue->CancelBuild(this);
		}

		if (GCardRepresentationAsyncQueue)
		{
			GCardRepresentationAsyncQueue->CancelBuild(this);
		}
	}
#endif

	// Tick base class to make progress on the streaming before calling HasPendingInitOrStreaming().
	if (!Super::IsReadyForFinishDestroy())
	{
		return false;
	}

	// Match BeginDestroy() by checking for HasPendingInitOrStreaming().
	if (HasPendingInitOrStreaming())
	{
		return false;
	}
	if (bRenderingResourcesInitialized)
	{
		ReleaseResources();
	}
	return ReleaseResourcesFence.IsFenceComplete();
}

int32 UStaticMesh::GetNumSectionsWithCollision() const
{
#if WITH_EDITORONLY_DATA
	int32 NumSectionsWithCollision = 0;

	if (GetRenderData() && GetRenderData()->LODResources.Num() > 0)
	{
		// Find how many sections have collision enabled
		const int32 UseLODIndex = FMath::Clamp(LODForCollision, 0, GetRenderData()->LODResources.Num() - 1);
		const FStaticMeshLODResources& CollisionLOD = GetRenderData()->LODResources[UseLODIndex];
		for (int32 SectionIndex = 0; SectionIndex < CollisionLOD.Sections.Num(); ++SectionIndex)
		{
			if (GetSectionInfoMap().Get(UseLODIndex, SectionIndex).bEnableCollision)
			{
				NumSectionsWithCollision++;
			}
		}
	}

	return NumSectionsWithCollision;
#else
	return 0;
#endif
}

void UStaticMesh::GetAssetRegistryTags(TArray<FAssetRegistryTag>& OutTags) const
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS;
	Super::GetAssetRegistryTags(OutTags);
	PRAGMA_ENABLE_DEPRECATION_WARNINGS;
}

void UStaticMesh::GetAssetRegistryTags(FAssetRegistryTagsContext Context) const
{
#if WITH_EDITORONLY_DATA
	Context.AddTag(FAssetRegistryTag("NaniteEnabled", IsNaniteEnabled() ? TEXT("True") : TEXT("False"), FAssetRegistryTag::TT_Alphabetical));
	Context.AddTag(FAssetRegistryTag("NaniteFallbackPercent", FString::Printf(TEXT("%.1f"), GetNaniteSettings().FallbackPercentTriangles * 100.0f), FAssetRegistryTag::TT_Numerical));

	if (GetAssetImportData())
	{
		Context.AddTag(FAssetRegistryTag(SourceFileTagName(), GetAssetImportData()->GetSourceData().ToJson(), FAssetRegistryTag::TT_Hidden));
	}
#endif

	// Avoid accessing properties being compiled, this function will get called again after compilation is finished.
	if (IsCompiling())
	{
		return;
	}

	int32 NumTriangles = 0;
	int32 NumVertices = 0;
	int32 NumUVChannels = 0;
	int32 NumLODs = 0;
#if WITH_EDITORONLY_DATA
	uint64 PhysicsSize = 0;
#endif

	if (GetRenderData() && GetRenderData()->LODResources.Num() > 0)
	{
		const FStaticMeshLODResources& LOD = GetRenderData()->LODResources[0];
		NumTriangles = LOD.IndexBuffer.GetNumIndices() / 3;
		NumVertices = LOD.VertexBuffers.StaticMeshVertexBuffer.GetNumVertices();
		NumUVChannels = LOD.VertexBuffers.StaticMeshVertexBuffer.GetNumTexCoords();
		NumLODs = GetRenderData()->LODResources.Num();
	}

	int32 NumSectionsWithCollision = GetNumSectionsWithCollision();

	int32 NumCollisionPrims = 0;
	if (GetBodySetup() != nullptr)
	{
		NumCollisionPrims = GetBodySetup()->AggGeom.GetElementCount();

#if WITH_EDITORONLY_DATA
		FResourceSizeEx EstimatedSize(EResourceSizeMode::EstimatedTotal);
		GetBodySetup()->GetResourceSizeEx(EstimatedSize);
		PhysicsSize = EstimatedSize.GetTotalMemoryBytes();
#endif
	}

	FBoxSphereBounds Bounds(ForceInit);
	if (GetRenderData())
	{
		Bounds = GetRenderData()->Bounds;
	}
	const FString ApproxSizeStr = FString::Printf(TEXT("%" INT64_FMT "x%" INT64_FMT "x%" INT64_FMT), FMath::RoundToInt(Bounds.BoxExtent.X * 2.0f), FMath::RoundToInt(Bounds.BoxExtent.Y * 2.0f), FMath::RoundToInt(Bounds.BoxExtent.Z * 2.0f));

	// Get name of default collision profile
	FName DefaultCollisionName = NAME_None;
	if(GetBodySetup() != nullptr)
	{
		DefaultCollisionName = GetBodySetup()->DefaultInstance.GetCollisionProfileName();
	}

	FString ComplexityString;
	if (GetBodySetup() != nullptr)
	{
		ComplexityString = LexToString((ECollisionTraceFlag)GetBodySetup()->GetCollisionTraceFlag());
	}

	int32 NumNaniteTriangles = GetNumNaniteTriangles();
	int32 NumNaniteVertices = GetNumNaniteVertices();

	int32 DistanceFieldSize = 0;

	if (GetRenderData() && GetRenderData()->LODResources.Num() > 0 && GetRenderData()->LODResources[0].DistanceFieldData != nullptr)
	{
		const FDistanceFieldVolumeData& VolumeData = *(GetRenderData()->LODResources[0].DistanceFieldData);

		DistanceFieldSize += VolumeData.GetResourceSizeBytes();
		DistanceFieldSize += VolumeData.StreamableMips.GetBulkDataSize();
	}

	uint64 EstimatedCompressedSize = 0;
	uint64 EstimatedNaniteCompressedSize = 0;
#if WITH_EDITORONLY_DATA
	if (GetRenderData())
	{
		EstimatedCompressedSize = (int32)GetRenderData()->EstimatedCompressedSize;
		EstimatedNaniteCompressedSize =  (int32)GetRenderData()->EstimatedNaniteTotalCompressedSize;
	}
#endif

	Context.AddTag(FAssetRegistryTag("NaniteTriangles", FString::FromInt(NumNaniteTriangles), FAssetRegistryTag::TT_Numerical));
	Context.AddTag(FAssetRegistryTag("NaniteVertices", FString::FromInt(NumNaniteVertices), FAssetRegistryTag::TT_Numerical));
	Context.AddTag(FAssetRegistryTag("Triangles", FString::FromInt(NumTriangles), FAssetRegistryTag::TT_Numerical) );
	Context.AddTag(FAssetRegistryTag("Vertices", FString::FromInt(NumVertices), FAssetRegistryTag::TT_Numerical) );
	Context.AddTag(FAssetRegistryTag("UVChannels", FString::FromInt(NumUVChannels), FAssetRegistryTag::TT_Numerical) );
	Context.AddTag(FAssetRegistryTag("Materials", FString::FromInt(GetStaticMaterials().Num()), FAssetRegistryTag::TT_Numerical) );
	Context.AddTag(FAssetRegistryTag(GetStaticMaterialsName(), ExportStaticMaterialsInfo(), FAssetRegistryTag::TT_Numerical));
	Context.AddTag(FAssetRegistryTag("ApproxSize", ApproxSizeStr, FAssetRegistryTag::TT_Dimensional) );
	Context.AddTag(FAssetRegistryTag("CollisionPrims", FString::FromInt(NumCollisionPrims), FAssetRegistryTag::TT_Numerical));
	Context.AddTag(FAssetRegistryTag("LODs", FString::FromInt(NumLODs), FAssetRegistryTag::TT_Numerical));
	Context.AddTag(FAssetRegistryTag("MinLOD", GetMinLOD().ToString(), FAssetRegistryTag::TT_Alphabetical));
	Context.AddTag(FAssetRegistryTag("QualityLevelMinLOD", GetQualityLevelMinLOD().ToString(), FAssetRegistryTag::TT_Alphabetical));
	Context.AddTag(FAssetRegistryTag("SectionsWithCollision", FString::FromInt(NumSectionsWithCollision), FAssetRegistryTag::TT_Numerical));
	Context.AddTag(FAssetRegistryTag("DefaultCollision", DefaultCollisionName.ToString(), FAssetRegistryTag::TT_Alphabetical));
	Context.AddTag(FAssetRegistryTag("CollisionComplexity", ComplexityString, FAssetRegistryTag::TT_Alphabetical));
	Context.AddTag(FAssetRegistryTag("DistanceFieldSize", FString::FromInt(DistanceFieldSize), FAssetRegistryTag::TT_Numerical, FAssetRegistryTag::TD_Memory));
	Context.AddTag(FAssetRegistryTag("EstTotalCompressedSize", FString::Printf(TEXT("%llu"), EstimatedCompressedSize), FAssetRegistryTag::TT_Numerical, FAssetRegistryTag::TD_Memory));
	Context.AddTag(FAssetRegistryTag("EstNaniteCompressedSize", FString::Printf(TEXT("%llu"), EstimatedNaniteCompressedSize), FAssetRegistryTag::TT_Numerical, FAssetRegistryTag::TD_Memory));
	
#if WITH_EDITORONLY_DATA
	Context.AddTag(FAssetRegistryTag("HasHiResMesh", IsHiResMeshDescriptionValid() ? TEXT("True") : TEXT("False"), FAssetRegistryTag::TT_Alphabetical));
	Context.AddTag(FAssetRegistryTag("PhysicsSize", FString::Printf(TEXT("%llu"), PhysicsSize), FAssetRegistryTag::TT_Numerical, FAssetRegistryTag::TD_Memory));
#endif

#if WITH_EDITOR	
	Context.AddTag(FAssetRegistryTag("BuildRequiredMemoryEstimate", FString::Printf(TEXT("%llu"), GetBuildRequiredMemoryEstimate()), FAssetRegistryTag::TT_Numerical, FAssetRegistryTag::TD_Memory));
#endif

	Super::GetAssetRegistryTags(Context);
}

#if WITH_EDITOR
void UStaticMesh::GetAssetRegistryTagMetadata(TMap<FName, FAssetRegistryTagMetadata>& OutMetadata) const
{
	Super::GetAssetRegistryTagMetadata(OutMetadata);

	OutMetadata.Add("NaniteEnabled",
		FAssetRegistryTagMetadata()
			.SetDisplayName(NSLOCTEXT("UStaticMesh", "NaniteEnabled_Label", "Nanite Enabled"))
		);

	OutMetadata.Add("NaniteFallbackPercent",
		FAssetRegistryTagMetadata()
			.SetDisplayName(NSLOCTEXT("UStaticMesh", "NaniteFallbackPercent_Label", "Nanite Fallback Percent"))
		);
	
	OutMetadata.Add("NaniteTriangles",
		FAssetRegistryTagMetadata()
			.SetDisplayName(NSLOCTEXT("UStaticMesh", "NaniteTriangles_Label", "Nanite Triangles"))
		);

	OutMetadata.Add("NaniteVertices",
		FAssetRegistryTagMetadata()
			.SetDisplayName(NSLOCTEXT("UStaticMesh", "NaniteVertices_Label", "Nanite Vertices"))
		);
	
	OutMetadata.Add("Triangles",
		FAssetRegistryTagMetadata()
			.SetDisplayName(NSLOCTEXT("UStaticMesh", "Triangles_Label", "Triangles"))
		);

	OutMetadata.Add("Vertices",
		FAssetRegistryTagMetadata()
			.SetDisplayName(NSLOCTEXT("UStaticMesh", "Vertices_Label", "Vertices"))
		);

	OutMetadata.Add("UVChannels",
		FAssetRegistryTagMetadata()
			.SetDisplayName(NSLOCTEXT("UStaticMesh", "UVChannels_Label", "UV Channels"))
		);

	OutMetadata.Add("Materials",
		FAssetRegistryTagMetadata()
			.SetDisplayName(NSLOCTEXT("UStaticMesh", "Materials_Label", "Materials"))
		);
	
	OutMetadata.Add(GetStaticMaterialsName(),
		FAssetRegistryTagMetadata()
			.SetDisplayName(NSLOCTEXT("UStaticMesh", "StaticMaterials_Label", "Static Materials"))
		);

	OutMetadata.Add("ApproxSize",
		FAssetRegistryTagMetadata()
			.SetDisplayName(NSLOCTEXT("UStaticMesh", "ApproxSize_Label", "Approximate Size"))
		);

	OutMetadata.Add("CollisionPrims",
		FAssetRegistryTagMetadata()
			.SetDisplayName(NSLOCTEXT("UStaticMesh", "CollisionPrims_Label", "Collision Primitives"))
			.SetTooltip(NSLOCTEXT("UStaticMesh", "CollisionPrims_Tooltip", "The number of collision primitives in the static mesh"))
			.SetImportantValue(TEXT("0"))
		);

	OutMetadata.Add("LODs",
		FAssetRegistryTagMetadata()
			.SetDisplayName(NSLOCTEXT("UStaticMesh", "LODs_Label", "LODs"))
		);

	OutMetadata.Add("MinLOD",
		FAssetRegistryTagMetadata()
			.SetDisplayName(NSLOCTEXT("UStaticMesh", "MinLOD_Label", "Min LOD"))
		);

	OutMetadata.Add("QualityLevelMinLOD",
		FAssetRegistryTagMetadata()
			.SetDisplayName(NSLOCTEXT("UStaticMesh", "QualityLevelMinLOD_Label", "Quality Level Min LOD"))
		);

	OutMetadata.Add("SectionsWithCollision",
		FAssetRegistryTagMetadata()
			.SetDisplayName(NSLOCTEXT("UStaticMesh", "SectionsWithCollision_Label", "Sections With Collision"))
		);

	OutMetadata.Add("DefaultCollision",
		FAssetRegistryTagMetadata()
			.SetDisplayName(NSLOCTEXT("UStaticMesh", "DefaultCollision_Label", "Default Collision"))
		);

	OutMetadata.Add("CollisionComplexity",
		FAssetRegistryTagMetadata()
			.SetDisplayName(NSLOCTEXT("UStaticMesh", "CollisionComplexity_Label", "Collision Complexity"))
		);

	OutMetadata.Add("DistanceFieldSize",
		FAssetRegistryTagMetadata()
			.SetDisplayName(NSLOCTEXT("UStaticMesh", "DistanceFieldSize_Label", "Distance Field Size"))
		);

	OutMetadata.Add("EstTotalCompressedSize",
		FAssetRegistryTagMetadata()
			.SetDisplayName(NSLOCTEXT("UStaticMesh", "EstTotalCompressedSize_Label", "Estimated Total Compressed Size"))
		);

	OutMetadata.Add("EstNaniteCompressedSize",
		FAssetRegistryTagMetadata()
			.SetDisplayName(NSLOCTEXT("UStaticMesh", "EstNaniteCompressedSize_Label", "Estimated Nanite Compressed Size"))
		);

	OutMetadata.Add("HasHiResMesh",
		FAssetRegistryTagMetadata()
			.SetDisplayName(NSLOCTEXT("UStaticMesh", "HasHiResMesh_Label", "Has HiRes Mesh"))
		);

	OutMetadata.Add("PhysicsSize",
		FAssetRegistryTagMetadata()
			.SetDisplayName(NSLOCTEXT("UStaticMesh", "PhysicsSize_Label", "Physics Size"))
		);
}
#endif


/*------------------------------------------------------------------------------
	FStaticMeshSourceModel
------------------------------------------------------------------------------*/


/*------------------------------------------------------------------------------
	FMeshSectionInfoMap
------------------------------------------------------------------------------*/

#if WITH_EDITORONLY_DATA
	
bool operator==(const FMeshSectionInfo& A, const FMeshSectionInfo& B)
{
	return A.MaterialIndex == B.MaterialIndex
		&& A.bCastShadow == B.bCastShadow
		&& A.bEnableCollision == B.bEnableCollision;
}

bool operator!=(const FMeshSectionInfo& A, const FMeshSectionInfo& B)
{
	return !(A == B);
}
	
static uint32 GetMeshMaterialKey(int32 LODIndex, int32 SectionIndex)
{
	return ((LODIndex & 0xffff) << 16) | (SectionIndex & 0xffff);
}

void FMeshSectionInfoMap::Clear()
{
	Map.Empty();
}

int32 FMeshSectionInfoMap::GetSectionNumber(int32 LODIndex) const
{
	int32 SectionCount = 0;
	for (auto kvp : Map)
	{
		if (((kvp.Key & 0xffff0000) >> 16) == LODIndex)
		{
			SectionCount++;
		}
	}
	return SectionCount;
}

bool FMeshSectionInfoMap::IsValidSection(int32 LODIndex, int32 SectionIndex) const
{
	uint32 Key = GetMeshMaterialKey(LODIndex, SectionIndex);
	return (Map.Find(Key) != nullptr);
}

FMeshSectionInfo FMeshSectionInfoMap::Get(int32 LODIndex, int32 SectionIndex) const
{
	uint32 Key = GetMeshMaterialKey(LODIndex, SectionIndex);
	const FMeshSectionInfo* InfoPtr = Map.Find(Key);
	if (InfoPtr == NULL)
	{
		Key = GetMeshMaterialKey(0, SectionIndex);
		InfoPtr = Map.Find(Key);
	}
	if (InfoPtr != NULL)
	{
		return *InfoPtr;
	}
	return FMeshSectionInfo(SectionIndex);
}

void FMeshSectionInfoMap::Set(int32 LODIndex, int32 SectionIndex, FMeshSectionInfo Info)
{
	uint32 Key = GetMeshMaterialKey(LODIndex, SectionIndex);
	Map.Add(Key, Info);
}

void FMeshSectionInfoMap::Remove(int32 LODIndex, int32 SectionIndex)
{
	uint32 Key = GetMeshMaterialKey(LODIndex, SectionIndex);
	Map.Remove(Key);
}

void FMeshSectionInfoMap::CopyFrom(const FMeshSectionInfoMap& Other)
{
	for (TMap<uint32,FMeshSectionInfo>::TConstIterator It(Other.Map); It; ++It)
	{
		Map.Add(It.Key(), It.Value());
	}
}

bool FMeshSectionInfoMap::AnySectionHasCollision(int32 LodIndex) const
{
	for (TMap<uint32,FMeshSectionInfo>::TConstIterator It(Map); It; ++It)
	{
		uint32 Key = It.Key();
		int32 KeyLODIndex = (int32)(Key >> 16);
		if (KeyLODIndex == LodIndex && It.Value().bEnableCollision)
		{
			return true;
		}
	}
	return false;
}

FArchive& operator<<(FArchive& Ar, FMeshSectionInfo& Info)
{
	Ar << Info.MaterialIndex;
	Ar << Info.bEnableCollision;
	Ar << Info.bCastShadow;
	return Ar;
}

void FMeshSectionInfoMap::Serialize(FArchive& Ar)
{
	Ar.UsingCustomVersion(FReleaseObjectVersion::GUID);
	Ar.UsingCustomVersion(FEditorObjectVersion::GUID);

	if ( Ar.CustomVer(FReleaseObjectVersion::GUID) < FReleaseObjectVersion::UPropertryForMeshSectionSerialize // Release-4.15 change
		&& Ar.CustomVer(FEditorObjectVersion::GUID) < FEditorObjectVersion::UPropertryForMeshSectionSerialize) // Dev-Editor change
	{
		Ar << Map;
	}
}

#endif // #if WITH_EDITORONLY_DATA

/**
 * Registers the mesh attributes required by the mesh description for a static mesh.
 */
void UStaticMesh::RegisterMeshAttributes(FMeshDescription& MeshDescription)
{
	FStaticMeshAttributes Attributes(MeshDescription);
	Attributes.Register();
}


#if WITH_EDITOR
FStaticMeshRenderData& UStaticMesh::GetPlatformStaticMeshRenderData(UStaticMesh* Mesh, const ITargetPlatform* Platform)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UStaticMesh::GetPlatformStaticMeshRenderData);

	check(Mesh && Mesh->GetRenderData());
	const FStaticMeshLODSettings& PlatformLODSettings = Platform->GetStaticMeshLODSettings();
	FString PlatformDerivedDataKey = BuildStaticMeshDerivedDataKey(
		BuildStaticMeshDerivedDataKeySuffix(Platform, Mesh, PlatformLODSettings.GetLODGroup(Mesh->GetLODGroup())));
	FStaticMeshRenderData* PlatformRenderData = Mesh->GetRenderData();

	if (Mesh->GetOutermost()->bIsCookedForEditor)
	{
		check(PlatformRenderData);
		return *PlatformRenderData;
	}

	while (PlatformRenderData)
	{
		if (PlatformRenderData->DerivedDataKey == PlatformDerivedDataKey)
		{
			PlatformRenderData->SupportedPlatforms.Add(Platform);
			break;
		}

		if (PlatformRenderData->SupportedPlatforms.Contains(Platform) && IsRunningCookCommandlet())
		{
			UE_LOGF(LogStaticMesh, Error,
				"Indeterminism in GetPlatformStaticMeshRenderData call to BuildStaticMeshDerivedDataKey. Using the previously computed RenderData even though the key is now different." "\n\tStaticMesh=%ls, Platform=%ls" "\n\tPreviousKey=%ls" "\n\tCurrentKey =%ls",
				*Mesh->GetPathName(), *Platform->PlatformName(),
				*PlatformRenderData->DerivedDataKey, *PlatformDerivedDataKey);
			break;
		}

		PlatformRenderData = PlatformRenderData->NextCachedRenderData.Get();
	}

	if (PlatformRenderData == NULL)
	{
		const bool bIsInGameThread = IsInGameThread();

		if (!Mesh->HasCachedNaniteAssemblyReferences() &&
			ensureMsgf(bIsInGameThread, TEXT("Nanite Assembly references were not pre-cached on the game thread. Build will likely fail")))
		{
			Mesh->RecacheNaniteAssemblyReferences();
		}

		// Cache render data for this platform and insert it in to the linked list.
		PlatformRenderData = new FStaticMeshRenderData();
		PlatformRenderData->Cache(Platform, Mesh, PlatformLODSettings);
		check(PlatformRenderData->DerivedDataKey == PlatformDerivedDataKey);
		PlatformRenderData->SupportedPlatforms.Add(Platform);
		Swap(PlatformRenderData->NextCachedRenderData, Mesh->GetRenderData()->NextCachedRenderData);
		Mesh->GetRenderData()->NextCachedRenderData = TUniquePtr<FStaticMeshRenderData>(PlatformRenderData);
	}
	check(PlatformRenderData);
	return *PlatformRenderData;
}

void UStaticMesh::WillNeverCacheCookedPlatformDataAgain()
{
}

void UStaticMesh::ClearCachedCookedPlatformData(const ITargetPlatform* TargetPlatform)
{
	if (!IsRunningCookCommandlet())
	{
		// Drop bulk data after serialization as editor streams from DDC and doesn't need it to be resident.
		// When running the cook commandlet multiple platforms might share the same FStaticMeshRenderData,
		// so we defer the dropping to ClearAllCachedCookedPlatformData.
		FStaticMeshRenderData& PlatformRenderData = GetPlatformStaticMeshRenderData(this, TargetPlatform);
		PlatformRenderData.NaniteResourcesPtr->DropBulkData();
	}
}

void UStaticMesh::ClearAllCachedCookedPlatformData()
{
	GetRenderData()->NextCachedRenderData.Reset();
	GetRenderData()->NaniteResourcesPtr->DropBulkData();
	GetRenderData()->NumCacheAttempts = 0;
}

void UStaticMesh::BeginCacheForCookedPlatformData(const ITargetPlatform* TargetPlatform)
{
}

bool UStaticMesh::IsCachedCookedPlatformDataLoaded(const ITargetPlatform* TargetPlatform)
{
	if (IsCompiling())
	{
		return false;
	}

	if ((GDistanceFieldAsyncQueue != nullptr && GDistanceFieldAsyncQueue->GetNumOutstandingTasks(this) > 0) ||
		(GCardRepresentationAsyncQueue != nullptr && GCardRepresentationAsyncQueue->GetNumOutstandingTasks(this) > 0))
	{
		return false;
	}

	FStaticMeshRenderData& PlatformRenderData = GetPlatformStaticMeshRenderData(this, TargetPlatform);

	bool bFailed = false;
	if (!PlatformRenderData.NaniteResourcesPtr->RebuildBulkDataFromCacheAsync(this, bFailed))
	{
		return false;
	}

	if (bFailed)
	{
		if (PlatformRenderData.NumCacheAttempts >= 2)
		{
			UE_LOGF(LogStaticMesh, Error,
				"Failed to recover Nanite streaming from DDC for '%ls', even after multiple attempts. " "Abandoning the attempt to build the static mesh renderdata; the mesh will be broken at runtime.", *GetPathName());
			return true;
		}
		UE_LOGF(LogStaticMesh, Log, "Failed to recover Nanite streaming from DDC for '%ls'. Rebuilding and retrying.", *GetPathName());

		// This should be a very rare event
		// For simplicity, just rebuild the entire RenderData, but preserve the CacheRetryData
		// TODO: Verify that move construction works for FPlatformRenderData, and use the move constructor rather than
		// calling CopyCacheRetryData twice..
		FStaticMeshRenderData CopyData;
		CopyData.CopyCacheRetryData(PlatformRenderData);

		PlatformRenderData.~FStaticMeshRenderData();
		new (&PlatformRenderData) FStaticMeshRenderData();

		PlatformRenderData.CopyCacheRetryData(CopyData);
		if (PlatformRenderData.NaniteResourcesPtr.IsValid())
		{
			PlatformRenderData.NaniteResourcesPtr->SetHasBuildFromDDCError(true);
		}
		PlatformRenderData.Cache(TargetPlatform, this, TargetPlatform->GetStaticMeshLODSettings());
		return false;
	}

	return true;
}

#if WITH_EDITORONLY_DATA
void FStaticMeshRenderData::CopyCacheRetryData(FStaticMeshRenderData& Other)
{
	SupportedPlatforms = Other.SupportedPlatforms;
	DerivedDataKey = Other.DerivedDataKey;
	NumCacheAttempts = Other.NumCacheAttempts;
}
#endif

#if WITH_EDITORONLY_DATA

bool UStaticMesh::LoadMeshDescription(int32 LodIndex, FMeshDescription& OutMeshDescription) const
{
	if (!IsSourceModelValid(LodIndex))
	{
		return false;
	}

	const FStaticMeshSourceModel& SourceModel = GetSourceModel(LodIndex);
	return SourceModel.LoadMeshDescription(OutMeshDescription);
}


bool UStaticMesh::CloneMeshDescription(int32 LodIndex, FMeshDescription& OutMeshDescription) const
{
	if (!IsSourceModelValid(LodIndex))
	{
		return false;
	}

	const FStaticMeshSourceModel& SourceModel = GetSourceModel(LodIndex);
	return SourceModel.CloneMeshDescription(OutMeshDescription);
}


FMeshDescription* UStaticMesh::GetMeshDescription(int32 LodIndex) const
{
	if (!IsSourceModelValid(LodIndex))
	{
		return nullptr;
	}

	// Require a const_cast here, because GetMeshDescription should ostensibly have const semantics,
	// but the lazy initialization (from the BulkData or the DDC) is a one-off event which breaks constness.
	UStaticMesh* MutableThis = const_cast<UStaticMesh*>(this);
	FStaticMeshSourceModel& SourceModel = MutableThis->GetSourceModel(LodIndex);

	return SourceModel.GetOrCacheMeshDescription();
}


bool UStaticMesh::IsMeshDescriptionValid(int32 LodIndex) const
{
	if (!IsSourceModelValid(LodIndex))
	{
		return false;
	}

	const FStaticMeshSourceModel& SourceModel = GetSourceModel(LodIndex);
	return SourceModel.IsMeshDescriptionValid();
}


FMeshDescription* UStaticMesh::CreateMeshDescription(int32 LodIndex)
{
	if (IsSourceModelValid(LodIndex))
	{
		FStaticMeshSourceModel& SourceModel = GetSourceModel(LodIndex);
		return SourceModel.CreateMeshDescription();
	}

	return nullptr;
}

FMeshDescription* UStaticMesh::CreateMeshDescription(int32 LodIndex, FMeshDescription InMeshDescription)
{
	FMeshDescription* NewMeshDescription = CreateMeshDescription(LodIndex);
	if (NewMeshDescription != nullptr)
	{
		*NewMeshDescription = MoveTemp(InMeshDescription);
	}

	return NewMeshDescription;
}


void UStaticMesh::CommitMeshDescription(int32 LodIndex, const FCommitMeshDescriptionParams& Params)
{
	// The source model must be created before calling this function
	check(IsSourceModelValid(LodIndex));

	FStaticMeshSourceModel& SourceModel = GetSourceModel(LodIndex);
	SourceModel.CommitMeshDescription(Params.bUseHashAsGuid);

	if (LodIndex == 0)
	{
		// For LOD0, cache the bounds of the mesh description in the static mesh for quick access
		if (IsNaniteAssembly())
		{
			CachedMeshDescriptionBounds.Reset(); // Nanite Assembly bounds must come from Nanite build
		}
		else if (SourceModel.GetCachedMeshDescription())
		{
			CachedMeshDescriptionBounds = SourceModel.GetCachedMeshDescription()->GetBounds();
		}
	}

	// This part is not thread-safe, so we give the caller the option of calling it manually from the mainthread
	if (Params.bMarkPackageDirty)
	{
		MarkPackageDirty();
	}
}

void UStaticMesh::ClearMeshDescription(int32 LodIndex)
{
	if (IsSourceModelValid(LodIndex))
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(UStaticMesh::ClearMeshDescription);

		FStaticMeshSourceModel& SourceModel = GetSourceModel(LodIndex);
		SourceModel.ClearMeshDescription();
	}
}


void UStaticMesh::ClearMeshDescriptions()
{
	for (int LODIndex = 0; LODIndex < GetNumSourceModels(); LODIndex++)
	{
		ClearMeshDescription(LODIndex);
	}
}


bool UStaticMesh::LoadHiResMeshDescription(FMeshDescription& OutMeshDescription) const
{
	const FStaticMeshSourceModel& SourceModel = GetHiResSourceModel();
	return SourceModel.LoadMeshDescription(OutMeshDescription);
}


bool UStaticMesh::CloneHiResMeshDescription(FMeshDescription& OutMeshDescription) const
{
	const FStaticMeshSourceModel& SourceModel = GetHiResSourceModel();
	return SourceModel.CloneMeshDescription(OutMeshDescription);
}


FMeshDescription* UStaticMesh::GetHiResMeshDescription() const
{
	// Require a const_cast here, because GetMeshDescription should ostensibly have const semantics,
	// but the lazy initialization (from the BulkData or the DDC) is a one-off event which breaks constness.
	UStaticMesh* MutableThis = const_cast<UStaticMesh*>(this);
	FStaticMeshSourceModel& SourceModel = MutableThis->GetHiResSourceModel();

	return SourceModel.GetOrCacheMeshDescription();
}


bool UStaticMesh::IsHiResMeshDescriptionValid() const
{
	const FStaticMeshSourceModel& SourceModel = GetHiResSourceModel();
	return SourceModel.IsMeshDescriptionValid();
}


FMeshDescription* UStaticMesh::CreateHiResMeshDescription()
{
	FStaticMeshSourceModel& SourceModel = GetHiResSourceModel();
	return SourceModel.CreateMeshDescription();
}


FMeshDescription* UStaticMesh::CreateHiResMeshDescription(FMeshDescription InMeshDescription)
{
	FMeshDescription* NewMeshDescription = CreateHiResMeshDescription();
	if (NewMeshDescription != nullptr)
	{
		*NewMeshDescription = MoveTemp(InMeshDescription);
	}

	return NewMeshDescription;
}


void UStaticMesh::CommitHiResMeshDescription(const FCommitMeshDescriptionParams& Params)
{
	FStaticMeshSourceModel& SourceModel = GetHiResSourceModel();
	SourceModel.CommitMeshDescription(Params.bUseHashAsGuid);

	// This part is not thread-safe, so we give the caller the option of calling it manually from the mainthread
	if (Params.bMarkPackageDirty)
	{
		MarkPackageDirty();
	}
}


void UStaticMesh::ClearHiResMeshDescription()
{
	FStaticMeshSourceModel& SourceModel = GetHiResSourceModel();
	SourceModel.ClearMeshDescription();
}


bool UStaticMesh::ModifyMeshDescription(int32 LodIndex, bool bAlwaysMarkDirty)
{
	FStaticMeshSourceModel& SourceModel = GetSourceModel(LodIndex);
	check(SourceModel.StaticMeshDescriptionBulkData != nullptr);
	return SourceModel.StaticMeshDescriptionBulkData->Modify(bAlwaysMarkDirty);
}


bool UStaticMesh::ModifyAllMeshDescriptions(bool bAlwaysMarkDirty)
{
	bool bResult = true;
	for (int LODIndex = 0; LODIndex < GetNumSourceModels(); LODIndex++)
	{
		if (!ModifyMeshDescription(LODIndex))
		{
			bResult = false;
		}
	}

	return bResult;
}


bool UStaticMesh::ModifyHiResMeshDescription(bool bAlwaysMarkDirty)
{
	FStaticMeshSourceModel& SourceModel = GetHiResSourceModel();
	check(SourceModel.StaticMeshDescriptionBulkData != nullptr);
	return SourceModel.StaticMeshDescriptionBulkData->Modify(bAlwaysMarkDirty);
}


void UStaticMesh::ConvertLegacySourceData()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UStaticMesh::CacheMeshData);

	// Generate MeshDescription source data in the DDC if no bulk data is present from the asset
	for (int32 LodIndex = 0; LodIndex < GetNumSourceModels(); ++LodIndex)
	{
		FStaticMeshSourceModel& SourceModel = GetSourceModel(LodIndex);

		// Legacy assets used to store their source data in the RawMeshBulkData
		// Migrate it to the new description if present
		SourceModel.ConvertRawMesh(LodIndex);
	}
}

bool UStaticMesh::AddUVChannel(int32 LODIndex)
{
	FMeshDescription* MeshDescription = GetMeshDescription(LODIndex);
	if (MeshDescription)
	{
		Modify();

		if (FStaticMeshOperations::AddUVChannel(*MeshDescription))
		{
			CommitMeshDescription(LODIndex);
			PostEditChange();

			return true;
		}
	}
	return false;
}

bool UStaticMesh::InsertUVChannel(int32 LODIndex, int32 UVChannelIndex)
{
	FMeshDescription* MeshDescription = GetMeshDescription(LODIndex);
	if (MeshDescription)
	{
		Modify();

		if (FStaticMeshOperations::InsertUVChannel(*MeshDescription, UVChannelIndex))
		{
			// Adjust the lightmap UV indices in the Build Settings to account for the new channel
			FMeshBuildSettings& LODBuildSettings = GetSourceModel(LODIndex).BuildSettings;
			if (UVChannelIndex <= LODBuildSettings.SrcLightmapIndex)
			{
				++LODBuildSettings.SrcLightmapIndex;
			}

			if (UVChannelIndex <= LODBuildSettings.DstLightmapIndex)
			{
				++LODBuildSettings.DstLightmapIndex;
			}

			if (UVChannelIndex <= GetLightMapCoordinateIndex())
			{
				SetLightMapCoordinateIndex(GetLightMapCoordinateIndex() + 1);
			}

			CommitMeshDescription(LODIndex);
			PostEditChange();

			return true;
		}
	}
	return false;
}

bool UStaticMesh::RemoveUVChannel(int32 LODIndex, int32 UVChannelIndex)
{
	FMeshDescription* MeshDescription = GetMeshDescription(LODIndex);
	if (MeshDescription)
	{
		FMeshBuildSettings& LODBuildSettings = GetSourceModel(LODIndex).BuildSettings;

		if (LODBuildSettings.bGenerateLightmapUVs)
		{
			if (UVChannelIndex == LODBuildSettings.SrcLightmapIndex)
			{
				UE_LOGF(LogStaticMesh, Error, "RemoveUVChannel: To remove the lightmap source UV channel, disable \"Generate Lightmap UVs\" in the Build Settings.");
				return false;
			}

			if (UVChannelIndex == LODBuildSettings.DstLightmapIndex)
			{
				UE_LOGF(LogStaticMesh, Error, "RemoveUVChannel: To remove the lightmap destination UV channel, disable \"Generate Lightmap UVs\" in the Build Settings.");
				return false;
			}
		}

		Modify();

		if (FStaticMeshOperations::RemoveUVChannel(*MeshDescription, UVChannelIndex))
		{
			// Adjust the lightmap UV indices in the Build Settings to account for the removed channel
			if (UVChannelIndex < LODBuildSettings.SrcLightmapIndex)
			{
				--LODBuildSettings.SrcLightmapIndex;
			}

			if (UVChannelIndex < LODBuildSettings.DstLightmapIndex)
			{
				--LODBuildSettings.DstLightmapIndex;
			}

			if (UVChannelIndex < GetLightMapCoordinateIndex())
			{
				SetLightMapCoordinateIndex(GetLightMapCoordinateIndex() - 1);
			}

			CommitMeshDescription(LODIndex);
			PostEditChange();

			return true;
		}
	}
	return false;
}

bool UStaticMesh::SetUVChannel(int32 LODIndex, int32 UVChannelIndex, const TMap<FVertexInstanceID, FVector2D>& TexCoords)
{
	FMeshDescription* MeshDescription = GetMeshDescription(LODIndex);
	if (!MeshDescription)
	{
		return false;
	}

	if (TexCoords.Num() < MeshDescription->VertexInstances().Num())
	{
		return false;
	}

	Modify();

	FStaticMeshAttributes Attributes(*MeshDescription);

	TMeshAttributesRef<FVertexInstanceID, FVector2f> UVs = Attributes.GetVertexInstanceUVs();
	for (const FVertexInstanceID VertexInstanceID : MeshDescription->VertexInstances().GetElementIDs())
	{
		if (const FVector2D* UVCoord = TexCoords.Find(VertexInstanceID))
		{
			UVs.Set(VertexInstanceID, UVChannelIndex, (FVector2f)*UVCoord);		// LWC_TODO: Precision loss? TexCoords should probably be passed as FVector2f.
		}
		else
		{
			ensureMsgf(false, TEXT("Tried to apply UV data that did not match the StaticMesh MeshDescription."));
		}
	}

	CommitMeshDescription(LODIndex);
	PostEditChange();

	return true;
}

bool UStaticMesh::SetUVChannel(int32 LODIndex, int32 UVChannelIndex, const TMap<FVertexInstanceID, FVector2f>& TexCoords)
{
	FMeshDescription* MeshDescription = GetMeshDescription(LODIndex);
	if (!MeshDescription)
	{
		return false;
	}

	if (TexCoords.Num() < MeshDescription->VertexInstances().Num())
	{
		return false;
	}

	Modify();

	FStaticMeshAttributes Attributes(*MeshDescription);

	TMeshAttributesRef<FVertexInstanceID, FVector2f> UVs = Attributes.GetVertexInstanceUVs();
	for (const FVertexInstanceID VertexInstanceID : MeshDescription->VertexInstances().GetElementIDs())
	{
		if (const FVector2f* UVCoord = TexCoords.Find(VertexInstanceID))
		{
			UVs.Set(VertexInstanceID, UVChannelIndex, *UVCoord);
		}
		else
		{
			ensureMsgf(false, TEXT("Tried to apply UV data that did not match the StaticMesh MeshDescription."));
		}
	}

	CommitMeshDescription(LODIndex);
	PostEditChange();

	return true;
}

#endif

int32 UStaticMesh::GetNumUVChannels(int32 LODIndex) const
{
	int32 NumUVChannels = 0;
#if WITH_EDITORONLY_DATA
	FMeshDescription* MeshDescription = GetMeshDescription(LODIndex);
	if (MeshDescription && !MeshDescription->IsEmpty())
	{
		FStaticMeshConstAttributes Attributes(*MeshDescription);
		NumUVChannels = Attributes.GetVertexInstanceUVs().GetNumChannels();
	}
#endif
	return NumUVChannels;
}

void UStaticMesh::CacheDerivedData()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UStaticMesh::CacheDerivedData);
	LLM_SCOPE_BYNAME(TEXT("AssetCompilation/StaticMesh"));

	const bool bIsInGameThread = IsInGameThread();

#if WITH_EDITORONLY_DATA
	ConvertLegacySourceData();
	
	if (!HasCachedNaniteAssemblyReferences() &&
		ensureMsgf(bIsInGameThread, TEXT("Nanite Assembly references were not pre-cached on the game thread. Build will likely fail")))
	{
		RecacheNaniteAssemblyReferences();
	}
#endif
	// Cache derived data for the running platform.
	ITargetPlatformManagerModule& TargetPlatformManager = GetTargetPlatformManagerRef();
	ITargetPlatform* RunningPlatform = TargetPlatformManager.GetRunningTargetPlatform();
	check(RunningPlatform);
	const FStaticMeshLODSettings& LODSettings = RunningPlatform->GetStaticMeshLODSettings();

	TUniquePtr<class FStaticMeshRenderData> OldMeshRenderData;
	if (GetRenderData())
	{
		// This is the responsibility of the caller to ensure this has been called
		// on the main thread when calling CacheDerivedData() from another thread.
		if (bIsInGameThread)
		{
			// Cancel any previous async builds before modifying RenderData
			// This can happen during import as the mesh is rebuilt redundantly
			if (GDistanceFieldAsyncQueue)
			{
				GDistanceFieldAsyncQueue->CancelBuild(this);
			}

			if (GCardRepresentationAsyncQueue)
			{
				GCardRepresentationAsyncQueue->CancelBuild(this);
			}
		}

		if (AreRenderingResourcesInitialized())
		{
			PRAGMA_DISABLE_DEPRECATION_WARNINGS
			OldMeshRenderData = MoveTemp(RenderData);
			PRAGMA_ENABLE_DEPRECATION_WARNINGS
		}
	}

	SetRenderData(MakeUnique<FStaticMeshRenderData>());
	GetRenderData()->Cache(RunningPlatform, this, LODSettings);

	// If the render data was initialized on the render thread, we need to clean it up
	if (OldMeshRenderData)
	{
		// on the game thread, this might use batching but on any other thread
		// it will send render commands to release the resource
		OldMeshRenderData->ReleaseResources();

		if (IsInGameThread())
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(OldMeshRenderDataFence);
			// On the game thread, it's currently hard to know if batching is being used
			// so we need to use a fence and wait.
			ReleaseResourcesFence.BeginFence();
			ReleaseResourcesFence.Wait();
		}
		else
		{
			// when batching isn't used, we can avoid the wait by just queueing the deletion
			// on the render thread.
			ENQUEUE_RENDER_COMMAND(DestroyOldMeshRenderData)(
				[OldMeshRenderData = MoveTemp(OldMeshRenderData)](FRHICommandListBase&) mutable
				{
				});
		}
	}
}

void UStaticMesh::PrepareDerivedDataForActiveTargetPlatforms()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UStaticMesh::PrepareDerivedDataForActiveTargetPlatforms);

	ITargetPlatformManagerModule& TargetPlatformManager = GetTargetPlatformManagerRef();
	ITargetPlatform* RunningPlatform = TargetPlatformManager.GetRunningTargetPlatform();
	const TArray<ITargetPlatform*>& TargetPlatforms = TargetPlatformManager.GetActiveTargetPlatforms();
	for (int32 PlatformIndex = 0; PlatformIndex < TargetPlatforms.Num(); ++PlatformIndex)
	{
		ITargetPlatform* Platform = TargetPlatforms[PlatformIndex];
		if (Platform != RunningPlatform)
		{
			GetPlatformStaticMeshRenderData(this, Platform);
		}
	}

	// Now that they are in local DDC cache, clear them to save memory
	// next time they are read it will be fast anyway.
	ClearAllCachedCookedPlatformData();
}

#endif // #if WITH_EDITORONLY_DATA

void UStaticMesh::CalculateExtendedBounds()
{
	FBoxSphereBounds Bounds(ForceInit);
#if WITH_EDITOR
	if (CachedMeshDescriptionBounds.IsSet())
	{
		Bounds = CachedMeshDescriptionBounds.GetValue();
	}
	else
#endif // #if WITH_EDITOR
	{
		if (GetRenderData())
		{
			Bounds = GetRenderData()->Bounds;
		}
	}

	// Only apply bound extension if necessary, as it will result in a larger bounding sphere radius than retrieved from the render data
	if (!GetNegativeBoundsExtension().IsZero() || !GetPositiveBoundsExtension().IsZero())
	{
		// Convert to Min and Max
		FVector Min = Bounds.Origin - Bounds.BoxExtent;
		FVector Max = Bounds.Origin + Bounds.BoxExtent;
		// Apply bound extensions
		Min -= GetNegativeBoundsExtension();
		Max += GetPositiveBoundsExtension();
		// Convert back to Origin, Extent and update SphereRadius
		Bounds.Origin = (Min + Max) / 2;
		Bounds.BoxExtent = (Max - Min) / 2;
		Bounds.SphereRadius = Bounds.BoxExtent.Size();
	}

	SetExtendedBounds(Bounds);
}

FName UStaticMesh::GetLODPathName(const UStaticMesh* Mesh, int32 LODIndex)
{
#if RHI_ENABLE_RESOURCE_INFO
	return FName(FString::Printf(TEXT("%s [LOD%d]"), Mesh ? *Mesh->GetPathName() : TEXT("UnknownStaticMesh"), LODIndex));
#else
	return NAME_None;
#endif
}

#if WITH_EDITORONLY_DATA
FUObjectAnnotationSparseBool GStaticMeshesThatNeedMaterialFixup;
#endif // #if WITH_EDITORONLY_DATA

LLM_DEFINE_TAG(StaticMesh_Serialize); // This is an important test case for LLM_DEFINE_TAG

/**
 *	UStaticMesh::Serialize
 */
void UStaticMesh::Serialize(FArchive& Ar)
{
	LLM_SCOPE_BYNAME(TEXT("StaticMesh/Serialize")); // This is an important test case for SCOPE_BYNAME with a matching LLM_DEFINE_TAG

	DECLARE_SCOPE_CYCLE_COUNTER( TEXT("UStaticMesh::Serialize"), STAT_StaticMesh_Serialize, STATGROUP_LoadTime );

	TRACE_CPUPROFILER_EVENT_SCOPE(UStaticMesh::Serialize);

	SCOPE_MS_ACCUMULATOR(STAT_StaticMesh_SerializeFull);

	// Skip serialization during compilation if told to do so.
	if (IsCompiling() && Ar.ShouldSkipCompilingAssets())
	{
		return;
	}

#if WITH_EDITOR
	// Don't try to finish compilation while still in the loader as we can't possibly have our mesh being compiled if it has not been loaded yet.
	if (!IsInAsyncLoadingThread() && !IsInParallelLoadingThread())
	{
		// Since UPROPERTY are accessed directly by offset during serialization instead of using accessors, 
		// the protection put in place to automatically finish compilation if a locked property is accessed will not work. 
		// We have no choice but to force finish the compilation here to avoid potential race conditions between 
		// async compilation and the serialization.
		FStaticMeshCompilingManager::FFinishCompilationOptions FinishCompilationOptions;
		FinishCompilationOptions.bIncludeDependentMeshes = Ar.IsLoading(); // Wait for dependent meshes if we're about to be editing ourselves
		FStaticMeshCompilingManager::Get().FinishCompilation({this}, FinishCompilationOptions);
	}
#endif

	{
		SCOPE_MS_ACCUMULATOR(STAT_StaticMesh_SerializeParent);
		Super::Serialize(Ar);
	}

	Ar.UsingCustomVersion(FEditorObjectVersion::GUID);
	Ar.UsingCustomVersion(FFrameworkObjectVersion::GUID);
	Ar.UsingCustomVersion(FRenderingObjectVersion::GUID);
	Ar.UsingCustomVersion(FReleaseObjectVersion::GUID);
	Ar.UsingCustomVersion(FUE5MainStreamObjectVersion::GUID);
	Ar.UsingCustomVersion(FUE5ReleaseStreamObjectVersion::GUID);
	Ar.UsingCustomVersion(FFortniteMainBranchObjectVersion::GUID);

	FStripDataFlags StripFlags( Ar );

	bool bCooked = Ar.IsCooking();
	Ar << bCooked;

#if WITH_EDITORONLY_DATA
	if (Ar.IsLoading() && Ar.UEVer() < VER_UE4_REMOVE_ZERO_TRIANGLE_SECTIONS)
	{
		GStaticMeshesThatNeedMaterialFixup.Set(this);
	}
#endif // #if WITH_EDITORONLY_DATA

	UBodySetup* LocalBodySetup = GetBodySetup();
	Ar << LocalBodySetup;
	SetBodySetup(LocalBodySetup);

	if (Ar.UEVer() >= VER_UE4_STATIC_MESH_STORE_NAV_COLLISION)
	{
		UNavCollisionBase* LocalNavCollision = GetNavCollision();
		Ar << LocalNavCollision;
		SetNavCollision(LocalNavCollision);

#if WITH_EDITOR
		if ((GetBodySetup() != nullptr) &&
			bHasNavigationData && 
			(GetNavCollision() == nullptr))
		{
			if (Ar.IsPersistent() && Ar.IsLoading() && (Ar.GetDebugSerializationFlags() & DSF_EnableCookerWarnings))
			{
				UE_LOGF(LogStaticMesh, Warning, "Serialized NavCollision but it was null (%ls) NavCollision will be created dynamicaly at cook time.  Please resave package %ls.", *GetName(), *GetOutermost()->GetPathName());
			}
		}
#endif
	}
#if WITH_EDITOR
	else if (bHasNavigationData && GetBodySetup() && (Ar.GetDebugSerializationFlags() & DSF_EnableCookerWarnings))
	{
		UE_LOGF(LogStaticMesh, Warning, "This StaticMeshes (%ls) NavCollision will be created dynamicaly at cook time.  Please resave %ls.", *GetName(), *GetOutermost()->GetPathName())
	}
#endif

	if(Ar.IsLoading() && Ar.CustomVer(FFrameworkObjectVersion::GUID) < FFrameworkObjectVersion::UseBodySetupCollisionProfile && GetBodySetup())
	{
		GetBodySetup()->DefaultInstance.SetCollisionProfileName(UCollisionProfile::BlockAll_ProfileName);
	}

#if WITH_EDITORONLY_DATA
	if( !StripFlags.IsEditorDataStripped() && Ar.IsLoading() )
	{
		if ( Ar.UEVer() < VER_UE4_DEPRECATED_STATIC_MESH_THUMBNAIL_PROPERTIES_REMOVED )
		{
			FRotator DummyThumbnailAngle;
			float DummyThumbnailDistance;
			Ar << DummyThumbnailAngle;
			Ar << DummyThumbnailDistance;
		}

		if (Ar.CustomVer(FRenderingObjectVersion::GUID) < FRenderingObjectVersion::DeprecatedHighResSourceMesh)
		{
			FString Deprecated_HighResSourceMeshName;
			uint32 Deprecated_HighResSourceMeshCRC;

			Ar << Deprecated_HighResSourceMeshName;
			Ar << Deprecated_HighResSourceMeshCRC;
		}
	}
#endif // #if WITH_EDITORONLY_DATA

	if( Ar.IsCountingMemory() )
	{
		// Include collision as part of memory used
		if (GetBodySetup())
		{
			GetBodySetup()->Serialize( Ar );
		}

		if ( GetNavCollision() )
		{
			GetNavCollision()->Serialize( Ar );
		}

		//TODO: Count these members when calculating memory used
		//Ar << ReleaseResourcesFence;
	}

	FGuid LocalLightingGuid = GetLightingGuid();
	Ar << LocalLightingGuid;
	SetLightingGuid(LocalLightingGuid);
	Ar << Sockets;

#if WITH_EDITOR
	if (!StripFlags.IsEditorDataStripped())
	{
		for (int32 i = 0; i < GetNumSourceModels(); ++i)
		{
			FStaticMeshSourceModel& SrcModel = GetSourceModel(i);
			SrcModel.SerializeBulkData(Ar, this);

			// Automatically detect assets saved before CL 16135278 which changed F16 to RTNE
			//	set them to bUseBackwardsCompatibleF16TruncUVs	
			if ( Ar.IsLoading() && Ar.CustomVer(FUE5MainStreamObjectVersion::GUID) < FUE5MainStreamObjectVersion::DirLightsAreAtmosphereLightsByDefault)
			{
				SrcModel.BuildSettings.bUseBackwardsCompatibleF16TruncUVs = true;
			}
		}

		if (Ar.CustomVer(FEditorObjectVersion::GUID) < FEditorObjectVersion::UPropertryForMeshSection)
		{
			GetSectionInfoMap().Serialize(Ar);
		}

		// Need to set a flag rather than do conversion in place as RenderData is not
		// created until postload and it is needed for bounding information
		SetRequiresLODDistanceConversion(Ar.UEVer() < VER_UE4_STATIC_MESH_SCREEN_SIZE_LODS);
		SetRequiresLODScreenSizeConversion(Ar.CustomVer(FFrameworkObjectVersion::GUID) < FFrameworkObjectVersion::LODsUseResolutionIndependentScreenSize);
	}
#endif // #if WITH_EDITOR

	// Inline the derived data for cooked builds. Never include render data when
	// counting memory as it is included by GetResourceSize.
	if (bCooked && !IsTemplate() && !Ar.IsCountingMemory())
	{	
		if (Ar.IsLoading())
		{
			SCOPE_MS_ACCUMULATOR(STAT_StaticMesh_RenderData);
			TUniquePtr<class FStaticMeshRenderData> LocalRenderData = MakeUnique<FStaticMeshRenderData>();
			LocalRenderData->Serialize(Ar, this, bCooked);
			if (FApp::CanEverRender() || !FPlatformProperties::RequiresCookedData())	// cooked assets can be loaded also in the headless editor commandlets
			{
				SetRenderData(MoveTemp(LocalRenderData));
			}
		}
#if WITH_EDITOR
		else if (Ar.IsSaving())
		{
			FStaticMeshRenderData& PlatformRenderData = GetPlatformStaticMeshRenderData(this, Ar.CookingTarget());
			PlatformRenderData.Serialize(Ar, this, bCooked);
		}
#endif
	}

	if (Ar.UEVer() >= VER_UE4_SPEEDTREE_STATICMESH)
	{
		bool bHasSpeedTreeWind = SpeedTreeWind.IsValid();
		Ar << bHasSpeedTreeWind;

		if (bHasSpeedTreeWind)
		{
			if (!SpeedTreeWind.IsValid())
			{
				SpeedTreeWind = TSharedPtr<FSpeedTreeWind>(new FSpeedTreeWind);
			}

			Ar << *SpeedTreeWind;
		}
	}

#if WITH_EDITORONLY_DATA
	if ( Ar.IsLoading() && Ar.UEVer() < VER_UE4_ASSET_IMPORT_DATA_AS_JSON && !GetAssetImportData())
	{
		// AssetImportData should always be valid
		SetAssetImportData(NewObject<UAssetImportData>(this, TEXT("AssetImportData")));
	}
	
	// SourceFilePath and SourceFileTimestamp were moved into a subobject
	if ( Ar.IsLoading() && Ar.UEVer() < VER_UE4_ADDED_FBX_ASSET_IMPORT_DATA && GetAssetImportData() )
	{
		// AssetImportData should always have been set up in the constructor where this is relevant
		FAssetImportInfo Info;
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		Info.Insert(FAssetImportInfo::FSourceFile(SourceFilePath_DEPRECATED));
		GetAssetImportData()->SourceData = MoveTemp(Info);
		
		SourceFilePath_DEPRECATED = TEXT("");
		SourceFileTimestamp_DEPRECATED = TEXT("");
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	if (Ar.IsLoading() && Ar.CustomVer(FRenderingObjectVersion::GUID) < FRenderingObjectVersion::DistanceFieldSelfShadowBias)
	{
		DistanceFieldSelfShadowBias = GetSourceModel(0).BuildSettings.DistanceFieldBias_DEPRECATED * 10.0f;
	}

	if (Ar.IsLoading() && Ar.CustomVer(FRenderingObjectVersion::GUID) < FRenderingObjectVersion::NaniteFallbackTarget)
	{
		if( GetNaniteSettings().FallbackRelativeError != 1.0f )
		{
			GetNaniteSettings().FallbackTarget = ENaniteFallbackTarget::RelativeError;
		}
		else if( GetNaniteSettings().FallbackPercentTriangles != 1.0f )
		{
			GetNaniteSettings().FallbackTarget = ENaniteFallbackTarget::PercentTriangles;
		}
	}

	if (Ar.CustomVer(FEditorObjectVersion::GUID) >= FEditorObjectVersion::RefactorMeshEditorMaterials)
#endif // WITH_EDITORONLY_DATA
	{
		TArray<FStaticMaterial>& StaticMaterialsArray = GetStaticMaterials();
		Ar << StaticMaterialsArray;
		UpdateStaticMaterialsInfoCache(StaticMaterialsArray);
	}
#if WITH_EDITORONLY_DATA
	else if (Ar.IsLoading())
	{
		TArray<UMaterialInterface*> Unique_Materials_DEPRECATED;
		TArray<FName> MaterialSlotNames;
		TArray<FStaticMaterial>& StaticMaterialsArray = GetStaticMaterials();
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		for (UMaterialInterface *MaterialInterface : Materials_DEPRECATED)
		{
			FName MaterialSlotName = MaterialInterface != nullptr ? MaterialInterface->GetFName() : NAME_None;
			int32 NameCounter = 1;
			if (MaterialInterface)
			{
				while (MaterialSlotName != NAME_None && MaterialSlotNames.Find(MaterialSlotName) != INDEX_NONE)
				{
					FString MaterialSlotNameStr = MaterialInterface->GetName() + TEXT("_") + FString::FromInt(NameCounter);
					MaterialSlotName = FName(*MaterialSlotNameStr);
					NameCounter++;
				}
			}
			MaterialSlotNames.Add(MaterialSlotName);
			StaticMaterialsArray.Add(FStaticMaterial(MaterialInterface, MaterialSlotName));
			int32 UniqueIndex = Unique_Materials_DEPRECATED.AddUnique(MaterialInterface);
#if WITH_EDITOR
			//We must cleanup the material list since we have a new way to build static mesh
			bCleanUpRedundantMaterialPostLoad = GetStaticMaterials().Num() > 1;
#endif
		}
		UpdateStaticMaterialsInfoCache(StaticMaterialsArray);
		Materials_DEPRECATED.Empty();
		PRAGMA_ENABLE_DEPRECATION_WARNINGS

	}
#endif // WITH_EDITORONLY_DATA


#if WITH_EDITOR
	bool bHasSpeedTreeWind = SpeedTreeWind.IsValid();
	if (Ar.CustomVer(FReleaseObjectVersion::GUID) < FReleaseObjectVersion::SpeedTreeBillboardSectionInfoFixup && bHasSpeedTreeWind)
	{
		// Ensure we have multiple tree LODs
		if (GetNumSourceModels() > 1)
		{
			// Look a the last LOD model and check its vertices
			const int32 LODIndex = GetNumSourceModels() - 1;
			FStaticMeshSourceModel& SourceModel = GetSourceModel(LODIndex);

			// If we get here, it is a very old version which still serializes as RawMesh.
			// Hence we can expect the RawMeshBulkData to be valid.
			// At this point it will not have been converted to MeshDescription.
			ensure(!SourceModel.RawMeshBulkData->IsEmpty());
			FRawMesh RawMesh;
			SourceModel.RawMeshBulkData->LoadRawMesh(RawMesh);

			// Billboard LOD is made up out of quads so check for this
			bool bQuadVertices = ((RawMesh.VertexPositions.Num() % 4) == 0);

			// If there is no section info for the billboard LOD make sure we add it
			uint32 Key = GetMeshMaterialKey(LODIndex, 0);
			bool bSectionInfoExists = GetSectionInfoMap().Map.Contains(Key);
			if (!bSectionInfoExists && bQuadVertices)
			{
				FMeshSectionInfo Info;
				// Assuming billboard material is added last
				Info.MaterialIndex = GetStaticMaterials().Num() - 1;
				GetSectionInfoMap().Set(LODIndex, 0, Info);
				GetOriginalSectionInfoMap().Set(LODIndex, 0, Info);
			}
		}
	}
#endif // WITH_EDITOR

#if WITH_EDITORONLY_DATA
	if (Ar.IsLoading() && Ar.CustomVer(FFortniteMainBranchObjectVersion::GUID) < FFortniteMainBranchObjectVersion::FixedTangentTransformForNonuniformBuildScale)
	{
		bool bHasNonUniformSourceModel = false;
		int32 NumSourceModels = GetNumSourceModels();
		for (int32 LODIndex = 0; LODIndex < NumSourceModels; ++LODIndex)
		{
			const FStaticMeshSourceModel& SourceModel = GetSourceModel(LODIndex);
			if (!SourceModel.BuildSettings.BuildScale3D.AllComponentsEqual())
			{
				bHasNonUniformSourceModel = true;
			}
		}
		// Only set the flag to use incorrect tangents if the asset had non-uniform scaling on a source model
		SetLegacyTangentScaling(bHasNonUniformSourceModel);
	}
#endif // WITH_EDITORONLY_DATA
}

bool UStaticMesh::IsPostLoadThreadSafe() const
{
	return false;
}

#if WITH_EDITOR

bool UStaticMesh::IsAsyncTaskComplete() const
{
	return AsyncTask == nullptr || AsyncTask->IsWorkDone();
}

void UStaticMesh::AcquireAsyncProperty(EStaticMeshAsyncProperties AsyncProperties)
{
	LockedProperties |= (uint32)AsyncProperties;
}

void UStaticMesh::ReleaseAsyncProperty(EStaticMeshAsyncProperties AsyncProperties)
{
	LockedProperties &= ~(uint32)AsyncProperties;
}

int64 UStaticMesh::GetBuildRequiredMemoryEstimate() const
{
	// We have to base our estimate on something accessible and known before the build, for now use the biggest bulk data size.
	int64 BiggestBulkDataSize = -1;
	if (GetHiResSourceModel().GetMeshDescriptionBulkData())
	{
		BiggestBulkDataSize = FMath::Max(BiggestBulkDataSize, GetHiResSourceModel().GetMeshDescriptionBulkData()->GetBulkDataSize());
	}

	for (const FStaticMeshSourceModel& Model : GetSourceModels())
	{
		if (Model.GetMeshDescriptionBulkData())
		{
			BiggestBulkDataSize = FMath::Max(BiggestBulkDataSize, Model.GetMeshDescriptionBulkData()->GetBulkDataSize());
		}

		if (Model.RawMeshBulkData)
		{
			BiggestBulkDataSize = FMath::Max(BiggestBulkDataSize, Model.RawMeshBulkData->GetBulkData().GetBulkDataSize());
		}
	}

	if (BiggestBulkDataSize == 0)
	{
		// When there is no bulk data size, the mesh isn't going to be built from source data.
		// Rather than zero, returns a small fixed size of 4MiB to account for the overhead.
		// For example, some engine meshes fall under this.
		return 1024 * 1024 * 4;
	}

	// Rough estimate of the memory that should be required to build that mesh.
	// Used -ddc=cold on the command line and opened some big meshes from the content browser one by one while
	// monitoring the editor commit memory spike generated by the compilation.
	// For smaller assets, -llm with LLM tags have been used and came down to mostly the same ratio.
	return BiggestBulkDataSize * 7;
}

#endif // WITH_EDITOR

//
//	UStaticMesh::PostLoad
//
void UStaticMesh::PostLoad()
{
	LLM_SCOPE(ELLMTag::StaticMesh);
	Super::PostLoad();

#if WITH_EDITOR
	FStaticMeshAsyncBuildScope AsyncBuildScope(this);
#endif

	// Scope for the FStaticMeshPostLoadContext
	{
		TUniquePtr<FStaticMeshPostLoadContext> Context = MakeUnique<FStaticMeshPostLoadContext>();
		BeginPostLoadInternal(*Context);

	#if WITH_EDITOR
		if (FStaticMeshCompilingManager::Get().IsAsyncCompilationAllowed(this))
		{
			FModuleManager::Get().LoadModuleChecked<IMeshUtilities>(TEXT("MeshUtilities"));
 
			// Load the mesh builder module in order to cache data for the running platform.
			ITargetPlatformManagerModule& TargetPlatformManager = GetTargetPlatformManagerRef();
			ITargetPlatform* RunningPlatform = TargetPlatformManager.GetRunningTargetPlatform();
			check(RunningPlatform);
			IMeshBuilderModule::GetForPlatform(RunningPlatform);

			// Additionally, load the mesh builder modules for any other platforms we care about.
			const TArray<ITargetPlatform*>& TargetPlatforms = TargetPlatformManager.GetActiveTargetPlatforms();
			for (ITargetPlatform* Platform : TargetPlatforms)
			{
				if (Platform != RunningPlatform)
				{
					IMeshBuilderModule::GetForPlatform(Platform);
				}
			}

			FQueuedThreadPool* StaticMeshThreadPool = FStaticMeshCompilingManager::Get().GetThreadPool();
			EQueuedWorkPriority BasePriority = FStaticMeshCompilingManager::Get().GetBasePriority(this);

			// We assume that complex collision mesh are small and fast to compute so stalling
			// on them should be fast. This is required to avoid stalling on the RenderData of the
			// ComplexCollisionMesh during the async build of this mesh.
			if (ComplexCollisionMesh && ComplexCollisionMesh->IsCompiling())
			{
				FStaticMeshCompilingManager::Get().FinishCompilation({ ComplexCollisionMesh });
			}

			AsyncTask = MakeUnique<FStaticMeshAsyncBuildTask>(this, MoveTemp(Context));
			if (!HasAnyDependenciesCompiling())
			{
				AsyncTask->StartBackgroundTask(StaticMeshThreadPool, BasePriority, EQueuedWorkFlags::DoNotRunInsideBusyWait, GetBuildRequiredMemoryEstimate(), TEXT("StaticMesh"));
			}

			if (IsNaniteAssembly())
			{
				FStaticMeshCompilingManager::Get().AddStaticMeshesWithDependencies({this});
			}
			else
			{
				FStaticMeshCompilingManager::Get().AddStaticMeshes({this});
			}
		}
		else
	#endif
		{
			ExecutePostLoadInternal(*Context);
			FinishPostLoadInternal(*Context);
		}
	}

	if (IsResourcePSOPrecachingEnabled() &&
		GetRenderData() != nullptr)
	{
		ERHIFeatureLevel::Type FeatureLevel = GetWorld() ? GetWorld()->GetFeatureLevel() : GMaxRHIFeatureLevel;
		EShaderPlatform ShaderPlatform = GetFeatureLevelShaderPlatform(FeatureLevel);
		bool bUseNanite = UseNanite(ShaderPlatform) && HasValidNaniteData();

		bool bAnySectionCastsShadows = false;
		TArray<int16, TInlineAllocator<2>> UsedMaterialIndices;
		for (FStaticMeshLODResources& LODRenderData : GetRenderData()->LODResources)
		{
			for (FStaticMeshSection& RenderSection : LODRenderData.Sections)
			{
				UsedMaterialIndices.AddUnique(RenderSection.MaterialIndex);
				bAnySectionCastsShadows |= RenderSection.bCastShadow;
			}
		}

		// Use default precache PSO params but take shadow casting into account and mark movable to have better coverage
		FPSOPrecacheParams PrecachePSOParams;
		PrecachePSOParams.bCastShadow = bAnySectionCastsShadows;
		PrecachePSOParams.SetMobility(EComponentMobility::Movable);

		TArray<const FVertexFactoryType*, TInlineAllocator<2>> CachingFactories;
		if (bUseNanite)
		{
			CachingFactories.Add(&FNaniteVertexFactory::StaticType);
		}
		else
		{
			CachingFactories.Add(&FLocalVertexFactory::StaticType);
		}

		for (uint16 MaterialIndex : UsedMaterialIndices)
		{
			UMaterialInterface* MaterialInterface = GetMaterial(MaterialIndex);
			if (MaterialInterface)
			{
				for (const FVertexFactoryType* VFType : CachingFactories)
				{
					if (IsComponentPSOPrecachingEnabled())
					{
						MaterialInterface->PrecachePSOs(VFType, PrecachePSOParams);
					}
				}
			}
		}
	}
}

#if WITH_EDITORONLY_DATA
void UStaticMesh::DeclareConstructClasses(TArray<FTopLevelAssetPath>& OutConstructClasses, const UClass* SpecificSubclass)
{
	Super::DeclareConstructClasses(OutConstructClasses, SpecificSubclass);
	OutConstructClasses.Add(FTopLevelAssetPath(UStaticMeshDescriptionBulkData::StaticClass()));
	OutConstructClasses.Add(FTopLevelAssetPath(TEXT("/Script/NavigationSystem.NavCollision")));
}
void UStaticMesh::AppendToClassSchema(FAppendToClassSchemaContext& Context)
{
	Super::AppendToClassSchema(Context);
	AppendMeshCardConstantsToClassSchema(Context);
	AppendDistanceFieldConstantsToClassSchema(Context);
}
#endif

void UStaticMesh::BeginPostLoadInternal(FStaticMeshPostLoadContext& Context)
{
#if WITH_EDITOR
	TRACE_CPUPROFILER_EVENT_SCOPE(UStaticMesh::BeginPostLoadInternal);

	// Make sure every static FString's are built and cached on the main thread
	// before trying to access it from multiple threads
	GetStaticMeshDerivedDataVersion();

	CheckForMissingShaderModels();

	// Lock all properties that should not be modified/accessed during async post-load
	AcquireAsyncProperty();

	// This scope allows us to use any locked properties without causing stalls
	FStaticMeshAsyncBuildScope AsyncBuildScope(this);

	FMeshBudgetProjectSettingsUtils::SetLodGroupForStaticMesh(this);

	if (GetNumSourceModels() > 0)
	{
		static const auto CVar = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.GenerateMeshDistanceFields"));

		if (CVar->GetValueOnAnyThread(true) != 0 || bGenerateMeshDistanceField)
		{
			for (int32 MaterialIndex = 0; MaterialIndex < GetStaticMaterials().Num(); MaterialIndex++)
			{
				UMaterialInterface* MaterialInterface = GetStaticMaterials()[MaterialIndex].MaterialInterface;
				if (MaterialInterface)
				{
					// Make sure dependency is postloaded
					MaterialInterface->ConditionalPostLoad();
				}
			}

			UStaticMesh* DistanceFieldReplacementMesh = GetSourceModel(0).BuildSettings.DistanceFieldReplacementMesh;

			if (DistanceFieldReplacementMesh)
			{
				DistanceFieldReplacementMesh->ConditionalPostLoad();
			}
		}
	}

	Context.bIsCookedForEditor = GetOutermost()->bIsCookedForEditor;
	if (!Context.bIsCookedForEditor)
	{
		// Needs to happen before 'CacheDerivedData'
		if (GetLinkerUEVersion() < VER_UE4_BUILD_SCALE_VECTOR)
		{
			int32 NumLODs = GetNumSourceModels();
			for (int32 LODIndex = 0; LODIndex < NumLODs; ++LODIndex)
			{
				FStaticMeshSourceModel& SrcModel = GetSourceModel(LODIndex);
				SrcModel.BuildSettings.BuildScale3D = FVector(SrcModel.BuildSettings.BuildScale_DEPRECATED);
			}
		}

		if (GetLinkerUEVersion() < VER_UE4_LIGHTMAP_MESH_BUILD_SETTINGS)
		{
			for (int32 i = 0; i < GetNumSourceModels(); i++)
			{
				GetSourceModel(i).BuildSettings.bGenerateLightmapUVs = false;
			}
		}

		if (GetLinkerUEVersion() < VER_UE4_MIKKTSPACE_IS_DEFAULT)
		{
			for (int32 i = 0; i < GetNumSourceModels(); ++i)
			{
				GetSourceModel(i).BuildSettings.bUseMikkTSpace = true;
			}
		}

		if (GetLinkerUEVersion() < VER_UE4_BUILD_MESH_ADJ_BUFFER_FLAG_EXPOSED)
		{
			FRawMesh TempRawMesh;
			uint32 TotalIndexCount = 0;

			for (int32 i = 0; i < GetNumSourceModels(); ++i)
			{
				// At this stage in loading, we have not yet converted a legacy asset's RawMesh to MeshDescription,
				// so access RawMesh directly instead of through the FStaticMeshSourceModel API,
				// because we don't want to perform an automatic conversion to MeshDescription at this point -
				// this will be done below in CacheDerivedData().
				// This is a path for legacy assets.
				if (!GetSourceModel(i).RawMeshBulkData->IsEmpty())
				{
					GetSourceModel(i).RawMeshBulkData->LoadRawMesh(TempRawMesh);
					TotalIndexCount += TempRawMesh.WedgeIndices.Num();
				}
			}
		}

		// The LODGroup update on load must happen before CacheDerivedData so we don't have to rebuild it after
		if (GUpdateMeshLODGroupSettingsAtLoad && GetLODGroup() != NAME_None)
		{
			SetLODGroup(GetLODGroup());
		}

		FModuleManager::Get().LoadModule("NaniteBuilder");
		IMeshUtilities& MeshUtilities = FModuleManager::Get().LoadModuleChecked<IMeshUtilities>("MeshUtilities");
		MeshUtilities.FixupMaterialSlotNames(this);

		const int32 CustomVersion = GetLinkerCustomVersion(FReleaseObjectVersion::GUID);
		if (GetLinkerUEVersion() < VER_UE4_STATIC_MESH_EXTENDED_BOUNDS || CustomVersion < FReleaseObjectVersion::StaticMeshExtendedBoundsFix)
		{
			// A stall is almost guaranteed during async build because mesh bounds are used extensively from many different places.
			Context.bShouldComputeExtendedBounds = true;
			UE_LOGF(LogStaticMesh, Log, "%ls should be resaved to improve async compilation performance.", *GetFullName());
		}
		else
		{
			// Do not stall on this property if it doesn't need to be recomputed after rebuild.
			ReleaseAsyncProperty(EStaticMeshAsyncProperties::ExtendedBounds);
		}
	}

	Context.bIsCookedForEditor = GetOutermost()->bIsCookedForEditor;
	Context.bNeedsMaterialFixup = GStaticMeshesThatNeedMaterialFixup.Get(this);
#endif
	
#if WITH_EDITORONLY_DATA
	Context.bNeedsMeshUVDensityFix = GetLinkerCustomVersion(FRenderingObjectVersion::GUID) < FRenderingObjectVersion::FixedMeshUVDensity;

	// If any, make sure the ComplexCollisionMesh is loaded before creating the NavCollision
	if (ComplexCollisionMesh && ComplexCollisionMesh != this)
	{
		ComplexCollisionMesh->ConditionalPostLoad();
	}

	// Cache the assembly references, but don't sync on the compile, the async compiler will schedule it appropriately
	RecacheNaniteAssemblyReferences(false);
#endif //WITH_EDITORONLY_DATA

	// We want to always have a BodySetup, its used for per-poly collision as well
	if (GetBodySetup() == nullptr)
	{
		CreateBodySetup();
	}

	// Make sure the object is created on the game-thread before going async
	if (bHasNavigationData && GetBodySetup() != nullptr)
	{
		if (GetNavCollision() == nullptr)
		{
			SetNavCollision(UNavCollisionBase::ConstructNew(*this));
		}
	}
}

void UStaticMesh::ExecutePostLoadInternal(FStaticMeshPostLoadContext& Context)
{
#if WITH_EDITOR
	TRACE_CPUPROFILER_EVENT_SCOPE(UStaticMesh::ExecutePostLoadInternal)

	if (!Context.bIsCookedForEditor)
	{
		// Generate and cache render data
		CacheDerivedData();
		if (IsNaniteAssembly())
		{
			CalculateExtendedBounds(); // in case of changes due to references
		}
	}

	if (UE::Tasks::FCancellationTokenScope::IsCurrentWorkCanceled())
	{
		return;
	}

	GetBodySetup()->CreatePhysicsMeshes();

	if (!Context.bIsCookedForEditor)
	{
		//Fix up the material to remove redundant material, this is needed since the material refactor where we do not have anymore copy of the materials
		//in the materials list
		if (GetRenderData() && bCleanUpRedundantMaterialPostLoad)
		{
			bool bMaterialChange = false;
			TArray<FStaticMaterial> CompactedMaterial;
			for (int32 LODIndex = 0; LODIndex < GetRenderData()->LODResources.Num(); ++LODIndex)
			{
				if (GetRenderData()->LODResources.IsValidIndex(LODIndex))
				{
					FStaticMeshLODResources& LOD = GetRenderData()->LODResources[LODIndex];
					const int32 NumSections = LOD.Sections.Num();
					for (int32 SectionIndex = 0; SectionIndex < NumSections; ++SectionIndex)
					{
						const int32 MaterialIndex = LOD.Sections[SectionIndex].MaterialIndex;
						if (GetStaticMaterials().IsValidIndex(MaterialIndex))
						{
							if (LODIndex == 0)
							{
								//We do not compact LOD 0 material
								CompactedMaterial.Add(GetStaticMaterials()[MaterialIndex]);
							}
							else
							{
								FMeshSectionInfo MeshSectionInfo = GetSectionInfoMap().Get(LODIndex, SectionIndex);
								int32 CompactedIndex = INDEX_NONE;
								if (GetStaticMaterials().IsValidIndex(MeshSectionInfo.MaterialIndex))
								{
									for (int32 CompactedMaterialIndex = 0; CompactedMaterialIndex < CompactedMaterial.Num(); ++CompactedMaterialIndex)
									{
										const FStaticMaterial& StaticMaterial = CompactedMaterial[CompactedMaterialIndex];
										if (GetStaticMaterials()[MeshSectionInfo.MaterialIndex].MaterialInterface == StaticMaterial.MaterialInterface)
										{
											CompactedIndex = CompactedMaterialIndex;
											break;
										}
									}
								}

								if (CompactedIndex == INDEX_NONE)
								{
									CompactedIndex = CompactedMaterial.Add(GetStaticMaterials()[MaterialIndex]);
								}
								if (MeshSectionInfo.MaterialIndex != CompactedIndex)
								{
									MeshSectionInfo.MaterialIndex = CompactedIndex;
									GetSectionInfoMap().Set(LODIndex, SectionIndex, MeshSectionInfo);
									bMaterialChange = true;
								}
							}
						}
					}
				}
			}
			//If we change some section material index or there is unused material, we must use the new compacted material list.
			if (bMaterialChange || CompactedMaterial.Num() < GetStaticMaterials().Num())
			{
				GetStaticMaterials().Empty(CompactedMaterial.Num());
				for (const FStaticMaterial& Material : CompactedMaterial)
				{
					GetStaticMaterials().Add(Material);
				}
				//Make sure the physic data is recompute
				if (GetBodySetup())
				{
					GetBodySetup()->InvalidatePhysicsData();
					UE_LOGF(LogStaticMesh, Warning, "Mesh %ls is recomputing physics on load. It must be resaved before it will cook deterministically. Please resave %ls.", *GetName(), *GetPathName());
				}
			}
			bCleanUpRedundantMaterialPostLoad = false;
		}

		if (GetRenderData() && Context.bNeedsMaterialFixup)
		{
			FixupZeroTriangleSections();
		}
	}

	if (GetRenderData())
	{
#if WITH_EDITORONLY_DATA
		FPerPlatformInt PerPlatformData = GetMinLOD();
		FPerQualityLevelInt PerQualityLevelData = GetQualityLevelMinLOD();

		// Convert PerPlatForm data to PerQuality if perQuality data have not been serialized.
		// Also test default value, since PerPLatformData can have Default !=0 and no PerPlaform data overrides.
		bool bConvertMinLODData = (PerQualityLevelData.PerQuality.Num() == 0 && PerQualityLevelData.Default == 0) && (PerPlatformData.PerPlatform.Num() != 0 || PerPlatformData.Default != 0);

		if (GEngine && GEngine->UseStaticMeshMinLODPerQualityLevels && bConvertMinLODData)
		{
			// get the platform groups
			const TArray<FName>& PlatformGroupNameArray = PlatformInfo::GetAllPlatformGroupNames();

			// Make sure all platforms and groups are known before updating any of them. Missing platforms would not properly be converted to PerQuality if some of them were known and others were not.
			bool bAllPlatformsKnown = true;
			for (const TPair<FName, int32>& Pair : PerPlatformData.PerPlatform)
			{
				const bool bIsPlatformGroup = PlatformGroupNameArray.Contains(Pair.Key);
				const bool bIsKnownPlatform = (FDataDrivenPlatformInfoRegistry::GetPlatformInfo(Pair.Key).IniPlatformName.IsNone() == false);
				if (!bIsPlatformGroup && !bIsKnownPlatform)
				{
					bAllPlatformsKnown = false;
					break;
				}
			}

			if (bAllPlatformsKnown)
			{
				//assign the default value
				PerQualityLevelData.Default = PerPlatformData.Default;

				// iterate over all platform and platform group entry: ex: XBOXONE = 2, CONSOLE=1, MOBILE = 3
				if (PerQualityLevelData.PerQuality.Num() == 0)
				{
					TMap<FName, int32> SortedPerPlatforms = PerPlatformData.PerPlatform;
					SortedPerPlatforms.KeySort([&](const FName& A, const FName& B) { return (PlatformGroupNameArray.Contains(A) > PlatformGroupNameArray.Contains(B)); });

					for (const TPair<FName, int32>& Pair : SortedPerPlatforms)
					{
						FSupportedQualityLevelArray QualityLevels;
						FString PlatformEntry = Pair.Key.ToString();

						QualityLevels = QualityLevelProperty::PerPlatformOverrideMapping(PlatformEntry, this);

						// we now have a range of quality levels supported on that platform or from that group
						// note: 
						// -platform group overrides will be applied first
						// -platform override sharing the same quality level will take the smallest MinLOD value between them
						// -ex: if XboxOne and PS4 maps to high and XboxOne MinLOD = 2 and PS4 MINLOD = 1, MINLOD 1 will be selected
						for (int32& QLKey : QualityLevels)
						{
							int32* Value = PerQualityLevelData.PerQuality.Find(QLKey);
							if (Value != nullptr)
							{
								*Value = FMath::Min(Pair.Value, *Value);
							}
							else
							{
								PerQualityLevelData.PerQuality.Add(QLKey, Pair.Value);
							}
						}
					}
				}
				SetQualityLevelMinLOD(PerQualityLevelData);
			}
		}
#endif
	}

	// check the MinLOD values are all within range
	FPerQualityLevelInt QualityLocalMinLOD;
	FPerPlatformInt LocalMinLOD;
	int32 MinAvailableLOD = INDEX_NONE;
	TArray<TPair<int32, FName>> InvalidMinLODs;
	CheckForValidMinLODs(QualityLocalMinLOD, LocalMinLOD, MinAvailableLOD, InvalidMinLODs);
	if (InvalidMinLODs.Num())
	{
		if (IsMinLodQualityLevelEnable())
		{
			SetQualityLevelMinLOD(QualityLocalMinLOD);
		}
		else
		{
			SetMinLOD(LocalMinLOD);
		}

		TArray<FText> MinLODErrors;
		for (const TPair<int32, FName>& InvalidMinLOD : InvalidMinLODs)
		{
			const int32 LODIdx = InvalidMinLOD.Key;
			const FName OverrideName = InvalidMinLOD.Value;

			FFormatNamedArguments Arguments;
			Arguments.Add(TEXT("MinLOD"), FText::AsNumber(LODIdx));
			Arguments.Add(TEXT("MinAvailLOD"), FText::AsNumber(MinAvailableLOD));
			Arguments.Add(TEXT("OverrideName"), FText::FromName(OverrideName));
			if (OverrideName.IsNone())
			{
				MinLODErrors.Add(FText::Format(LOCTEXT("LoadError_BadMinLOD_Fixed", "Min LOD value of {MinLOD} is out of range 0..{MinAvailLOD} and has been adjusted to {MinAvailLOD}. Please verify and resave the asset."), Arguments));
			}
			else
			{
				MinLODErrors.Add(FText::Format(LOCTEXT("LoadError_BadMinLODWithOverride_Fixed", "Min LOD override of {MinLOD} for {OverrideName} is out of range 0..{MinAvailLOD} and has been adjusted to {MinAvailLOD}. Please verify and resave the asset."), Arguments));
			}
		}

		if (IsRunningCommandlet())
		{
			for (const FText& MinLODError : MinLODErrors)
			{
				UE_ASSET_LOG(LogStaticMesh, Warning, this, TEXT("%s"), *MinLODError.ToString());
			}
		}
		else
		{
			TSharedRef<FUObjectToken> TokenRef = FUObjectToken::Create(this);
			Async(
				EAsyncExecution::TaskGraphMainThread,
				// No choice to MoveTemp here, the SharedRef is not thread safe so it cannot
				// be copied to another thread, only moved.
				[Token = MoveTemp(TokenRef), MinAvailableLOD, MinLODErrors]()
				{
					for (const FText& MinLODError : MinLODErrors)
					{
						FMessageLog("LoadErrors").Warning()
							->AddToken(Token)
							->AddToken(FTextToken::Create(MinLODError));
					}

					FMessageLog("LoadErrors").Open();
				}
			);
		}
	}

#endif // #if WITH_EDITOR

#if WITH_EDITORONLY_DATA
	if (Context.bNeedsMeshUVDensityFix)
	{
		UpdateUVChannelData(true);
	}
#endif

	EnforceLightmapRestrictions();

#if WITH_EDITOR
	if (Context.bShouldComputeExtendedBounds)
	{
		CalculateExtendedBounds();
		ReleaseAsyncProperty(EStaticMeshAsyncProperties::ExtendedBounds);
	}

	// Those are going to apply modifications to RenderData and should execute before we release
	// the lock and send the InitResources.

	// Conversion of LOD distance need valid bounds it must be call after the extended Bounds fixup
	// Only required in an editor build as other builds process this in a different place
	if (GetRequiresLODDistanceConversion())
	{
		// Convert distances to Display Factors
		ConvertLegacyLODDistance();
	}

	if (GetRequiresLODScreenSizeConversion())
	{
		// Convert screen area to screen size
		ConvertLegacyLODScreenArea();
	}

	//Always redo the whole SectionInfoMap to be sure it contain only valid data
	//This will reuse everything valid from the just serialize SectionInfoMap.
	FMeshSectionInfoMap TempOldSectionInfoMap = GetSectionInfoMap();
	GetSectionInfoMap().Clear();
	if (GetRenderData())
	{
		for (int32 LODResourceIndex = 0; LODResourceIndex < GetRenderData()->LODResources.Num(); ++LODResourceIndex)
		{
			FStaticMeshLODResources& LOD = GetRenderData()->LODResources[LODResourceIndex];
			for (int32 SectionIndex = 0; SectionIndex < LOD.Sections.Num(); ++SectionIndex)
			{
				if (TempOldSectionInfoMap.IsValidSection(LODResourceIndex, SectionIndex))
				{
					FMeshSectionInfo Info = TempOldSectionInfoMap.Get(LODResourceIndex, SectionIndex);
					if (GetStaticMaterials().IsValidIndex(Info.MaterialIndex))
					{
						//Reuse the valid data that come from the serialize
						GetSectionInfoMap().Set(LODResourceIndex, SectionIndex, Info);
					}
					else
					{
						//Use the render data material index, but keep the flags (collision, shadow...)
						const int32 MaterialIndex = LOD.Sections[SectionIndex].MaterialIndex;
						if (GetStaticMaterials().IsValidIndex(MaterialIndex))
						{
							Info.MaterialIndex = MaterialIndex;
							GetSectionInfoMap().Set(LODResourceIndex, SectionIndex, Info);
						}
					}
				}
				else
				{
					//Create a new SectionInfoMap from the render data
					const int32 MaterialIndex = LOD.Sections[SectionIndex].MaterialIndex;
					if (GetStaticMaterials().IsValidIndex(MaterialIndex))
					{
						GetSectionInfoMap().Set(LODResourceIndex, SectionIndex, FMeshSectionInfo(MaterialIndex));
					}
				}
				//Make sure the OriginalSectionInfoMap has some information, the post load only add missing slot, this data should be set when importing/re-importing the asset
				if (!GetOriginalSectionInfoMap().IsValidSection(LODResourceIndex, SectionIndex))
				{
					GetOriginalSectionInfoMap().Set(LODResourceIndex, SectionIndex, GetSectionInfoMap().Get(LODResourceIndex, SectionIndex));
				}
			}
		}
	}

	// Additionally cache derived data for any other platforms we care about.
	// This must be done after the sectioninfomap fixups to make sure the DDC key matches
	// the one the cooker will generate during save.
	if (!Context.bIsCookedForEditor)
	{
		PrepareDerivedDataForActiveTargetPlatforms();
	}

	ReleaseAsyncProperty(EStaticMeshAsyncProperties::SectionInfoMap);
	ReleaseAsyncProperty(EStaticMeshAsyncProperties::OriginalSectionInfoMap);

	// Release cached mesh descriptions until they are loaded on demand
	ClearMeshDescriptions();

	if (GetNavCollision())
	{
		// Physics meshes need to be ready to gather the collision in Setup().
		GetBodySetup()->CreatePhysicsMeshes();
		GetNavCollision()->Setup(GetBodySetup());
	}

	ReleaseAsyncProperty(EStaticMeshAsyncProperties::SourceModels);
	ReleaseAsyncProperty(EStaticMeshAsyncProperties::HiResSourceModel);
#endif // #if WITH_EDITOR
}

#if WITH_EDITOR
void UStaticMesh::CheckForValidMinLODs(FPerQualityLevelInt& QualityLocalMinLOD, FPerPlatformInt& LocalMinLOD, int32& OutMinAvailableLOD, TArray<TPair<int32, FName>>& OutInvalidMinLODs) const
{
	const FStaticMeshRenderData* LocalRenderData = GetRenderData();
	if (!LocalRenderData)
	{
		return;
	}

	OutMinAvailableLOD = FMath::Max<int32>(LocalRenderData->LODResources.Num() - 1, 0);

	auto CheckValidMinLOD = [LocalRenderData, OutMinAvailableLOD, &OutInvalidMinLODs](int32& LODIdx, FName OverrideName)
	{
		if (!LocalRenderData->LODResources.IsValidIndex(LODIdx))
		{
			OutInvalidMinLODs.Emplace(LODIdx, OverrideName);
			LODIdx = OutMinAvailableLOD;
		}
	};

	if (IsMinLodQualityLevelEnable())
	{
		QualityLocalMinLOD = GetQualityLevelMinLOD();
		CheckValidMinLOD(QualityLocalMinLOD.Default, NAME_None);

		for (TMap<int32, int32>::TIterator It(QualityLocalMinLOD.PerQuality); It; ++It)
		{
			CheckValidMinLOD(It.Value(), QualityLevelProperty::QualityLevelToFName(It.Key()));
		}
	}
	else
	{
		LocalMinLOD = GetMinLOD();
		CheckValidMinLOD(LocalMinLOD.Default, NAME_None);

		for (TMap<FName, int32>::TIterator It(LocalMinLOD.PerPlatform); It; ++It)
		{
			CheckValidMinLOD(It.Value(), It.Key());
		}
	}
}

void UStaticMesh::CheckForMissingShaderModels()
{
#if PLATFORM_WINDOWS || PLATFORM_LINUX
	static bool bWarnedAboutMissingShaderModel = false;
	if (GIsEditor && IsNaniteEnabled() && !bWarnedAboutMissingShaderModel)
	{
		auto CopySM6Format = [](const TCHAR* ShaderFormatName, const TArray<FString>& SrcArray, TArray<FString>& DstArray)
		{
			if (SrcArray.Contains(ShaderFormatName))
			{
				DstArray.AddUnique(ShaderFormatName);
			}
		};

		TArray<FString> D3D11TargetedShaderFormats;
		TArray<FString> D3D12TargetedShaderFormats;
		TArray<FString> WindowsVulkanTargetedShaderFormats;
		TArray<FString> WindowsTargetedRHIs;
		TArray<FString> LinuxVulkanTargetedShaderFormats;
		TArray<FString> LinuxTargetedRHIs;

#if PLATFORM_WINDOWS
		// Gather all Windows shader format settings
		{
			GConfig->GetArray(TEXT("/Script/WindowsTargetPlatform.WindowsTargetSettings"), TEXT("TargetedRHIs"), WindowsTargetedRHIs, GEngineIni);

			// If using Vulkan in Windows, warn about Vulkan settings
			if (IsVulkanPlatform(GMaxRHIShaderPlatform))
			{
				GConfig->GetArray(TEXT("/Script/WindowsTargetPlatform.WindowsTargetSettings"), TEXT("VulkanTargetedShaderFormats"), WindowsVulkanTargetedShaderFormats, GEngineIni);
				CopySM6Format(TEXT("SF_VULKAN_SM6"), WindowsTargetedRHIs, WindowsVulkanTargetedShaderFormats);
			}
			else
			{
				GConfig->GetArray(TEXT("/Script/WindowsTargetPlatform.WindowsTargetSettings"), TEXT("D3D11TargetedShaderFormats"), D3D11TargetedShaderFormats, GEngineIni);
				GConfig->GetArray(TEXT("/Script/WindowsTargetPlatform.WindowsTargetSettings"), TEXT("D3D12TargetedShaderFormats"), D3D12TargetedShaderFormats, GEngineIni);
				CopySM6Format(TEXT("PCD3D_SM6"), WindowsTargetedRHIs, D3D12TargetedShaderFormats);
			}
		}
#elif PLATFORM_LINUX
		// Gather all Linux shader format settings
		GConfig->GetArray(TEXT("/Script/LinuxTargetPlatform.LinuxTargetSettings"), TEXT("VulkanTargetedShaderFormats"), LinuxVulkanTargetedShaderFormats, GEngineIni);
		GConfig->GetArray(TEXT("/Script/LinuxTargetPlatform.LinuxTargetSettings"), TEXT("TargetedRHIs"), LinuxTargetedRHIs, GEngineIni);
		CopySM6Format(TEXT("SF_VULKAN_SM6"), LinuxTargetedRHIs, LinuxVulkanTargetedShaderFormats);
#elif PLATFORM_MAC
		// TODO: Gather all Mac shader format settings
#endif

		const bool bProjectUsesD3D = (D3D11TargetedShaderFormats.Num() + D3D12TargetedShaderFormats.Num()) > 0;
		const bool bProjectMissingD3DSM6 = (bProjectUsesD3D && !D3D12TargetedShaderFormats.Contains(TEXT("PCD3D_SM6")));

		const bool bProjectUsesWindowsVulkan = (WindowsVulkanTargetedShaderFormats.Num() > 0);
		const bool bProjectMissingWindowsVulkanSM6 = (bProjectUsesWindowsVulkan && !WindowsVulkanTargetedShaderFormats.Contains(TEXT("SF_VULKAN_SM6")));

		const bool bProjectUsesLinuxVulkan = (LinuxTargetedRHIs.Num() > 0) || (LinuxVulkanTargetedShaderFormats.Num() > 0);
		const bool bProjectMissingLinuxVulkanSM6 = (bProjectUsesLinuxVulkan && !LinuxVulkanTargetedShaderFormats.Contains(TEXT("SF_VULKAN_SM6")));

		if (bProjectMissingD3DSM6 || bProjectMissingWindowsVulkanSM6 || bProjectMissingLinuxVulkanSM6)
		{
			bWarnedAboutMissingShaderModel = true;

			auto DismissNotification = [This = TStrongObjectPtr(this)]
			{
				if (UStaticMesh* Mesh = This.Get())
				{
					if (TSharedPtr<SNotificationItem> NotificationPin = Mesh->ShaderModelNotificationPtr.Pin())
					{
						NotificationPin->SetCompletionState(SNotificationItem::CS_None);
						NotificationPin->ExpireAndFadeout();
						Mesh->ShaderModelNotificationPtr.Reset();
					}
				}
			};

			auto OpenProjectSettings = []()
			{
				FGlobalTabmanager::Get()->TryInvokeTab(FName("ProjectSettings"));
			};

			FNotificationInfo Info(LOCTEXT("NeedProjectSettings", "Missing Project Settings!"));
			Info.bFireAndForget = false;
			Info.FadeOutDuration = 0.0f;
			Info.ExpireDuration = 0.0f;
			Info.WidthOverride = FOptionalSize();

			Info.ButtonDetails.Add(FNotificationButtonInfo(
				LOCTEXT("GuidelineDismiss", "Dismiss"),
				LOCTEXT("GuidelineDismissTT", "Dismiss this notification."),
				FSimpleDelegate::CreateLambda(DismissNotification),
				SNotificationItem::CS_None));

			Info.Text = LOCTEXT("NeedProjectSettings", "Missing Project Settings!");
			Info.HyperlinkText = LOCTEXT("ProjectSettingsHyperlinkText", "Open Project Settings");
			Info.Hyperlink = FSimpleDelegate::CreateLambda(OpenProjectSettings);

			if (bProjectMissingD3DSM6)
			{
				Info.SubText = LOCTEXT("NaniteNeedsSM6Setting", "Shader Model 6 (SM6) is required to use Nanite assets. Please enable this in:\n  Project Settings -> Platforms -> Windows -> D3D12 Targeted Shader Formats\nNanite assets will not display properly until this is enabled.");
			}
			else if (bProjectMissingWindowsVulkanSM6)
			{
				Info.SubText = LOCTEXT("NaniteNeedsSM6VulkanSM6WindowsSetting", "Shader Model 6 (SM6) is required to use Nanite assets. Please enable this in:\n  Project Settings -> Platforms -> Windows -> Vulkan Targeted Shader Formats\nNanite assets will not display properly in Vulkan on Windows until this is enabled.");
			}
			else if (bProjectMissingLinuxVulkanSM6)
			{
				Info.SubText = LOCTEXT("NaniteNeedsSM6VulkanSM6LinuxSetting", "Shader Model 6 (SM6) is required to use Nanite assets. Please enable this in:\n  Project Settings -> Platforms -> Linux -> Targeted RHIs\nNanite assets will not display properly in Vulkan on Linux until this is enabled.");
			}

			ShaderModelNotificationPtr = FSlateNotificationManager::Get().AddNotification(Info);
		}
	}
#endif // PLATFORM_WINDOWS || PLATFORM_LINUX
}
#endif

void UStaticMesh::FinishPostLoadInternal(FStaticMeshPostLoadContext& Context)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UStaticMesh::FinishPostLoad);

	if( FApp::CanEverRender() && !HasAnyFlags(RF_ClassDefaultObject) )
	{
		// InitResources will send commands to other threads that will
		// use our RenderData, we must mark it as ready to be used since
		// we're not going to modify it anymore
		ReleaseAsyncProperty(EStaticMeshAsyncProperties::RenderData);
		InitResources();
	}
	else
	{
		// Update any missing data when cooking.
		UpdateUVChannelData(false);
#if WITH_EDITOR
		if (GetRenderData())
		{
			GetRenderData()->ResolveSectionInfo(this);
		}
#endif
		ReleaseAsyncProperty(EStaticMeshAsyncProperties::RenderData);
	}

	ReleaseAsyncProperty();
}

void UStaticMesh::BuildFromMeshDescription(const FMeshDescription& MeshDescription, FStaticMeshLODResources& LODResources)
{
	FStaticMeshConstAttributes MeshDescriptionAttributes(MeshDescription);

	// Fill vertex buffers

	int32 NumVertexInstances = MeshDescription.VertexInstances().GetArraySize();
	int32 NumTriangles = MeshDescription.Triangles().Num();

	if (NumVertexInstances == 0 || NumTriangles == 0)
	{
		return;
	}

	TArray<FStaticMeshBuildVertex> StaticMeshBuildVertices;
	StaticMeshBuildVertices.SetNum(NumVertexInstances);

	TVertexAttributesConstRef<FVector3f> VertexPositions = MeshDescriptionAttributes.GetVertexPositions();
	TVertexInstanceAttributesConstRef<FVector3f> VertexInstanceNormals = MeshDescriptionAttributes.GetVertexInstanceNormals();
	TVertexInstanceAttributesConstRef<FVector3f> VertexInstanceTangents = MeshDescriptionAttributes.GetVertexInstanceTangents();
	TVertexInstanceAttributesConstRef<float> VertexInstanceBinormalSigns = MeshDescriptionAttributes.GetVertexInstanceBinormalSigns();
	TVertexInstanceAttributesConstRef<FVector4f> VertexInstanceColors = MeshDescriptionAttributes.GetVertexInstanceColors();
	TVertexInstanceAttributesConstRef<FVector2f> VertexInstanceUVs = MeshDescriptionAttributes.GetVertexInstanceUVs();

	for (FVertexInstanceID VertexInstanceID : MeshDescription.VertexInstances().GetElementIDs())
	{
		FStaticMeshBuildVertex& StaticMeshVertex = StaticMeshBuildVertices[VertexInstanceID.GetValue()];

		StaticMeshVertex.Position = VertexPositions[MeshDescription.GetVertexInstanceVertex(VertexInstanceID)];
		StaticMeshVertex.TangentX = VertexInstanceTangents[VertexInstanceID];
		StaticMeshVertex.TangentY = FVector3f::CrossProduct(VertexInstanceNormals[VertexInstanceID], VertexInstanceTangents[VertexInstanceID]).GetSafeNormal() * VertexInstanceBinormalSigns[VertexInstanceID];
		StaticMeshVertex.TangentZ = VertexInstanceNormals[VertexInstanceID];

		for (int32 UVIndex = 0; UVIndex < VertexInstanceUVs.GetNumChannels(); ++UVIndex)
		{
			StaticMeshVertex.UVs[UVIndex] = VertexInstanceUVs.Get(VertexInstanceID, UVIndex);
		}
	}

	bool bHasVertexColors = false;
	if (VertexInstanceColors.IsValid())
	{
		for (FVertexInstanceID VertexInstanceID : MeshDescription.VertexInstances().GetElementIDs())
		{
			FStaticMeshBuildVertex& StaticMeshVertex = StaticMeshBuildVertices[VertexInstanceID.GetValue()];

			FLinearColor Color(VertexInstanceColors[VertexInstanceID]);
			if (Color != FLinearColor::White)
			{
				bHasVertexColors = true;
				StaticMeshVertex.Color = Color.ToFColor(true);
			}
			else
			{
				StaticMeshVertex.Color = FColor::White;
			}
		}
	}

	LODResources.VertexBuffers.PositionVertexBuffer.Init(StaticMeshBuildVertices);

	FStaticMeshVertexBufferFlags StaticMeshVertexBufferFlags;
	StaticMeshVertexBufferFlags.bNeedsCPUAccess = true;
	StaticMeshVertexBufferFlags.bUseBackwardsCompatibleF16TruncUVs = false;
	LODResources.VertexBuffers.StaticMeshVertexBuffer.Init(StaticMeshBuildVertices, VertexInstanceUVs.GetNumChannels(), StaticMeshVertexBufferFlags);

	LODResources.bHasColorVertexData = bHasVertexColors;
	FColorVertexBuffer& ColorVertexBuffer = LODResources.VertexBuffers.ColorVertexBuffer;
	if (bHasVertexColors)
	{
		ColorVertexBuffer.Init(StaticMeshBuildVertices);
	}
	else
	{
		ColorVertexBuffer.InitFromSingleColor(FColor::White, NumVertexInstances);
	}

	// Fill index buffer and sections array

	int32 NumPolygonGroups = MeshDescription.PolygonGroups().Num();

	TPolygonGroupAttributesConstRef<FName> MaterialSlotNames = MeshDescriptionAttributes.GetPolygonGroupMaterialSlotNames();

	TArray<uint32> IndexBuffer;
	IndexBuffer.SetNumZeroed(NumTriangles * 3);

	FStaticMeshSectionArray& Sections = LODResources.Sections;

	int32 SectionIndex = 0;
	int32 IndexBufferIndex = 0;
	EIndexBufferStride::Type IndexBufferStride = EIndexBufferStride::Force16Bit;

	for (FPolygonGroupID PolygonGroupID : MeshDescription.PolygonGroups().GetElementIDs())
	{
		// Skip empty polygon groups - we do not want to build empty sections
		if (MeshDescription.GetNumPolygonGroupPolygons(PolygonGroupID) == 0)
		{
			continue;
		}

		FStaticMeshSection& Section = Sections.AddDefaulted_GetRef();
		Section.FirstIndex = IndexBufferIndex;

		int32 TriangleCount = 0;
		uint32 MinVertexIndex = TNumericLimits<uint32>::Max();
		uint32 MaxVertexIndex = TNumericLimits<uint32>::Min();

		for (FTriangleID TriangleID : MeshDescription.GetPolygonGroupTriangles(PolygonGroupID))
		{
			for (FVertexInstanceID TriangleVertexInstanceIDs : MeshDescription.GetTriangleVertexInstances(TriangleID))
			{
				uint32 VertexIndex = static_cast<uint32>(TriangleVertexInstanceIDs.GetValue());
				MinVertexIndex = FMath::Min(MinVertexIndex, VertexIndex);
				MaxVertexIndex = FMath::Max(MaxVertexIndex, VertexIndex);
				IndexBuffer[IndexBufferIndex] = VertexIndex;
				IndexBufferIndex++;
			}

			TriangleCount++;
		}

		Section.NumTriangles = TriangleCount;
		Section.MinVertexIndex = MinVertexIndex;
		Section.MaxVertexIndex = MaxVertexIndex;

		const int32 MaterialIndex = GetStaticMaterials().IndexOfByPredicate(
			[&MaterialSlotName = MaterialSlotNames[PolygonGroupID]](const FStaticMaterial& StaticMaterial) { return StaticMaterial.MaterialSlotName == MaterialSlotName; }
		);

		Section.MaterialIndex = MaterialIndex;
		Section.bEnableCollision = true;
		Section.bCastShadow = true;

		if (MaxVertexIndex > TNumericLimits<uint16>::Max())
		{
			IndexBufferStride = EIndexBufferStride::Force32Bit;
		}

		SectionIndex++;
	}
	check(IndexBufferIndex == NumTriangles * 3);

	LODResources.IndexBuffer.SetIndices(IndexBuffer, IndexBufferStride);
	LODResources.bHasColorVertexData = true;

	const bool bDepthOnlyIndexBuffers = !!CVarSupportDepthOnlyIndexBuffers.GetValueOnAnyThread();
	const bool bReversedIndexBuffers = !!CVarSupportReversedIndexBuffers.GetValueOnAnyThread();

	// Fill depth only index buffer

	TArray<uint32> DepthOnlyIndexBuffer(IndexBuffer);
	if (bDepthOnlyIndexBuffers)
	{
		for (uint32& Index : DepthOnlyIndexBuffer)
		{
			// Compress all vertex instances into the same instance for each vertex
			Index = MeshDescription.GetVertexVertexInstanceIDs(MeshDescription.GetVertexInstanceVertex(FVertexInstanceID(Index)))[0].GetValue();
		}

		LODResources.bHasDepthOnlyIndices = true;
		LODResources.DepthOnlyIndexBuffer.SetIndices(DepthOnlyIndexBuffer, IndexBufferStride);
		LODResources.DepthOnlyNumTriangles = DepthOnlyIndexBuffer.Num() / 3;
	}

	// Fill reversed index buffer
	if (bReversedIndexBuffers)
	{
		TArray<uint32> ReversedIndexBuffer(IndexBuffer);
		for (int32 ReversedIndexBufferIndex = 0; ReversedIndexBufferIndex < IndexBuffer.Num(); ReversedIndexBufferIndex += 3)
		{
			Swap(ReversedIndexBuffer[ReversedIndexBufferIndex + 0], ReversedIndexBuffer[ReversedIndexBufferIndex + 2]);
		}

		LODResources.AdditionalIndexBuffers = new FAdditionalStaticMeshIndexBuffers();
		LODResources.bHasReversedIndices = true;
		LODResources.AdditionalIndexBuffers->ReversedIndexBuffer.SetIndices(ReversedIndexBuffer, IndexBufferStride);

		if (bDepthOnlyIndexBuffers)
		{
			// Fill reversed depth index buffer
			TArray<uint32> ReversedDepthOnlyIndexBuffer(DepthOnlyIndexBuffer);
			for (int32 ReversedIndexBufferIndex = 0; ReversedIndexBufferIndex < IndexBuffer.Num(); ReversedIndexBufferIndex += 3)
			{
				Swap(ReversedDepthOnlyIndexBuffer[ReversedIndexBufferIndex + 0], ReversedDepthOnlyIndexBuffer[ReversedIndexBufferIndex + 2]);
			}

			LODResources.bHasReversedDepthOnlyIndices = true;
			LODResources.AdditionalIndexBuffers->ReversedDepthOnlyIndexBuffer.SetIndices(ReversedDepthOnlyIndexBuffer, IndexBufferStride);
		}
	}
}

void UStaticMesh::ExportStaticMeshLOD(const FStaticMeshLODResources& InLODResources, FMeshDescription& OutMeshDescription) const
{
	const int32 NumWedges = InLODResources.IndexBuffer.GetNumIndices();
	const int32 NumVertexPositions = InLODResources.VertexBuffers.PositionVertexBuffer.GetNumVertices();
	const int32 NumFaces = NumWedges / 3;

	OutMeshDescription.Empty();

	if (NumVertexPositions <= 0 || InLODResources.VertexBuffers.StaticMeshVertexBuffer.GetNumVertices() <= 0)
	{
		return;
	}

	FStaticMeshAttributes Attributes(OutMeshDescription);
	Attributes.Register();

	TVertexAttributesRef<FVector3f> VertexPositions = Attributes.GetVertexPositions();
	TEdgeAttributesRef<bool> EdgeHardnesses = Attributes.GetEdgeHardnesses();
	TPolygonGroupAttributesRef<FName> PolygonGroupImportedMaterialSlotNames = Attributes.GetPolygonGroupMaterialSlotNames();
	TVertexInstanceAttributesRef<FVector3f> VertexInstanceNormals = Attributes.GetVertexInstanceNormals();
	TVertexInstanceAttributesRef<FVector3f> VertexInstanceTangents = Attributes.GetVertexInstanceTangents();
	TVertexInstanceAttributesRef<float> VertexInstanceBinormalSigns = Attributes.GetVertexInstanceBinormalSigns();
	TVertexInstanceAttributesRef<FVector4f> VertexInstanceColors = Attributes.GetVertexInstanceColors();
	TVertexInstanceAttributesRef<FVector2f> VertexInstanceUVs = Attributes.GetVertexInstanceUVs();

	OutMeshDescription.ReserveNewVertices(NumVertexPositions);
	OutMeshDescription.ReserveNewVertexInstances(NumWedges);
	OutMeshDescription.ReserveNewPolygons(NumFaces);
	OutMeshDescription.ReserveNewEdges(NumWedges);

	const int32 NumTexCoords = InLODResources.VertexBuffers.StaticMeshVertexBuffer.GetNumTexCoords();
	VertexInstanceUVs.SetNumChannels(NumTexCoords);

	TMap<int32, FPolygonGroupID> SectionToPolygonGroupMap;

	for (int32 SectionIndex = 0; SectionIndex < InLODResources.Sections.Num(); ++SectionIndex)
	{
		const FStaticMeshSection& Section = InLODResources.Sections[SectionIndex];

		// Skip empty sections
		if (Section.NumTriangles == 0)
		{
			continue;
		}

		FPolygonGroupID CurrentPolygonGroupID = OutMeshDescription.CreatePolygonGroup();
		SectionToPolygonGroupMap.Emplace(SectionIndex, CurrentPolygonGroupID);

		if (GetStaticMaterials().IsValidIndex(Section.MaterialIndex))
		{
			PolygonGroupImportedMaterialSlotNames[CurrentPolygonGroupID] = GetStaticMaterials()[Section.MaterialIndex].ImportedMaterialSlotName;
		}
		else
		{
			PolygonGroupImportedMaterialSlotNames[CurrentPolygonGroupID] = FName(*(TEXT("MeshMergeMaterial_") + FString::FromInt(SectionIndex)));
		}
	}

	//Create the vertex
	for (int32 VertexIndex = 0; VertexIndex < NumVertexPositions; ++VertexIndex)
	{
		FVertexID VertexID = OutMeshDescription.CreateVertex();
		VertexPositions[VertexID] = InLODResources.VertexBuffers.PositionVertexBuffer.VertexPosition(VertexIndex);
	}

	//Create the vertex instances
	for (int32 TriangleIndex = 0; TriangleIndex < NumFaces; ++TriangleIndex)
	{
		FPolygonGroupID CurrentPolygonGroupID = INDEX_NONE;
		for (int32 SectionIndex = 0; SectionIndex < InLODResources.Sections.Num(); ++SectionIndex)
		{
			const FStaticMeshSection& Section = InLODResources.Sections[SectionIndex];
			uint32 BeginTriangle = Section.FirstIndex / 3;
			uint32 EndTriangle = BeginTriangle + Section.NumTriangles;
			if ((uint32)TriangleIndex >= BeginTriangle && (uint32)TriangleIndex < EndTriangle)
			{
				CurrentPolygonGroupID = SectionToPolygonGroupMap[SectionIndex];
				break;
			}
		}
		check(CurrentPolygonGroupID != INDEX_NONE);

		FVertexID VertexIDs[3];
		TArray<FVertexInstanceID> VertexInstanceIDs;
		VertexInstanceIDs.SetNum(3);

		for (int32 Corner = 0; Corner < 3; ++Corner)
		{
			int32 WedgeIndex = InLODResources.IndexBuffer.GetIndex(TriangleIndex * 3 + Corner);
			FVertexID VertexID(WedgeIndex);
			FVertexInstanceID VertexInstanceID = OutMeshDescription.CreateVertexInstance(VertexID);
			VertexIDs[Corner] = VertexID;
			VertexInstanceIDs[Corner] = VertexInstanceID;

			//NTBs
			FVector TangentX = FVector4(InLODResources.VertexBuffers.StaticMeshVertexBuffer.VertexTangentX(WedgeIndex));
			FVector TangentY = FVector(InLODResources.VertexBuffers.StaticMeshVertexBuffer.VertexTangentY(WedgeIndex));
			FVector TangentZ = FVector4(InLODResources.VertexBuffers.StaticMeshVertexBuffer.VertexTangentZ(WedgeIndex));
			VertexInstanceTangents[VertexInstanceID] = (FVector3f)TangentX;
			VertexInstanceBinormalSigns[VertexInstanceID] = GetBasisDeterminantSign(TangentX, TangentY, TangentZ);
			VertexInstanceNormals[VertexInstanceID] = (FVector3f)TangentZ;

			// Vertex colors
			if (InLODResources.VertexBuffers.ColorVertexBuffer.GetNumVertices() > 0)
			{
				VertexInstanceColors[VertexInstanceID] = FLinearColor(InLODResources.VertexBuffers.ColorVertexBuffer.VertexColor(WedgeIndex));
			}
			else
			{
				VertexInstanceColors[VertexInstanceID] = FLinearColor::White;
			}

			//Tex coord
			for (int32 TexCoodIdx = 0; TexCoodIdx < NumTexCoords; ++TexCoodIdx)
			{
				VertexInstanceUVs.Set(VertexInstanceID, TexCoodIdx, InLODResources.VertexBuffers.StaticMeshVertexBuffer.GetVertexUV(WedgeIndex, TexCoodIdx));
			}
		}
		//Create a polygon from this triangle
		const FPolygonID NewPolygonID = OutMeshDescription.CreatePolygon(CurrentPolygonGroupID, VertexInstanceIDs);
	}
}


UStaticMeshDescription* UStaticMesh::CreateStaticMeshDescription(UObject* Outer)
{
	if (Outer == nullptr)
	{
		Outer = GetTransientPackage();
	}

	UStaticMeshDescription* StaticMeshDescription = NewObject<UStaticMeshDescription>(Outer, NAME_None, RF_Transient);
	StaticMeshDescription->RegisterAttributes();
	return StaticMeshDescription;
}


UStaticMeshDescription* UStaticMesh::GetStaticMeshDescription(int32 LODIndex)
{
#if WITH_EDITOR
	if (LODIndex < GetNumSourceModels())
	{
		GetSourceModel(LODIndex).GetOrCacheMeshDescription();
		return GetSourceModel(LODIndex).GetCachedStaticMeshDescription();
	}
#endif
	return nullptr;
}


void UStaticMesh::BuildFromStaticMeshDescriptions(const TArray<UStaticMeshDescription*>& StaticMeshDescriptions, bool bBuildSimpleCollision, bool bFastBuild)
{
	TArray<const FMeshDescription*> MeshDescriptions;
	MeshDescriptions.Reserve(StaticMeshDescriptions.Num());

	for (UStaticMeshDescription* StaticMeshDescription : StaticMeshDescriptions)
	{
		MeshDescriptions.Emplace(&StaticMeshDescription->GetMeshDescription());
	}

	FBuildMeshDescriptionsParams Params;
	Params.bBuildSimpleCollision = bBuildSimpleCollision;
	Params.bFastBuild = bFastBuild;
	BuildFromMeshDescriptions(MeshDescriptions, Params);
}


bool UStaticMesh::BuildFromMeshDescriptions(const TArray<const FMeshDescription*>& MeshDescriptions, const FBuildMeshDescriptionsParams& Params)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UStaticMesh::BuildFromMeshDescriptions);

	const int32 NewNumLODs = FMath::Min(MeshDescriptions.Num(), MAX_STATIC_MESH_LODS);

	// Set up
	NeverStream = true;

#if !WITH_EDITOR
	// In non-Editor builds, we can only perform fast mesh builds
	check(Params.bFastBuild);
#endif
	if (Params.bFastBuild)
	{
		bDoFastBuild = true;
	}

	check(Params.bCommitMeshDescription || Params.bFastBuild);

	TOptional<FStaticMeshComponentRecreateRenderStateContext> RecreateRenderStateContext;
	
	bool bNewMesh = true;
	if (AreRenderingResourcesInitialized())
	{
		bNewMesh = false;
		const bool bInvalidateLighting = true;
		const bool bRefreshBounds = true;
		RecreateRenderStateContext = FStaticMeshComponentRecreateRenderStateContext(this, bInvalidateLighting, bRefreshBounds);
		ReleaseResources();
		ReleaseResourcesFence.Wait();
	}

	#if WITH_EDITOR
	if (Params.bCommitMeshDescription)
	{
		FCommitMeshDescriptionParams CommitParams;
		CommitParams.bMarkPackageDirty = Params.bMarkPackageDirty;
		CommitParams.bUseHashAsGuid = Params.bUseHashAsGuid;

		SetNumSourceModels(NewNumLODs);
		for (int32 LODIndex = 0; LODIndex < NewNumLODs; LODIndex++)
		{
			FMeshBuildSettings& LODBuildSettings = GetSourceModel(LODIndex).BuildSettings;
			if (Params.PerLODOverrides.IsValidIndex(LODIndex))
			{
				const FBuildMeshDescriptionsLODParams& LODParams = Params.PerLODOverrides[LODIndex];
				LODBuildSettings.bUseHighPrecisionTangentBasis = LODParams.bUseHighPrecisionTangentBasis;
				LODBuildSettings.bUseFullPrecisionUVs = LODParams.bUseFullPrecisionUVs;
			}
			CreateMeshDescription(LODIndex, *MeshDescriptions[LODIndex]);
			CommitMeshDescription(LODIndex, CommitParams);
		}
	}
	#endif

	SetRenderData(MakeUnique<FStaticMeshRenderData>());
	GetRenderData()->AllocateLODResources(NewNumLODs);

	FStaticMeshLODResourcesArray& LODResourcesArray = GetRenderData()->LODResources;
	for (int32 LODIndex = 0; LODIndex < LODResourcesArray.Num(); ++LODIndex)
	{
		LODResourcesArray[LODIndex].IndexBuffer.TrySetAllowCPUAccess(bAllowCPUAccess || Params.bAllowCpuAccess);
		if (Params.PerLODOverrides.IsValidIndex(LODIndex))
		{
			const FBuildMeshDescriptionsLODParams& LODParams = Params.PerLODOverrides[LODIndex];
			LODResourcesArray[LODIndex].VertexBuffers.StaticMeshVertexBuffer.SetUseHighPrecisionTangentBasis(LODParams.bUseHighPrecisionTangentBasis);
			LODResourcesArray[LODIndex].VertexBuffers.StaticMeshVertexBuffer.SetUseFullPrecisionUVs(LODParams.bUseFullPrecisionUVs);
		}
	}

	// Build render data from each mesh description

#if WITH_EDITOR
	if (Params.bFastBuild)
#endif
	{
		for (int32 LODIndex = 0; LODIndex < NewNumLODs; LODIndex++)
		{
			check(MeshDescriptions[LODIndex] != nullptr);
			FStaticMeshLODResources& LODResources = GetRenderData()->LODResources[LODIndex];

			BuildFromMeshDescription(*MeshDescriptions[LODIndex], LODResources);
		}

		bool bInitializeRayTracingRepresentation = bSupportRayTracing;

#if !WITH_EDITOR
		// non-editor build -> only initialize ray tracing representation if ray tracing is allowed
		// editor build -> initialize because we don't know TargetPlatform, but might be stripped later during serialization
		if (!IsRayTracingAllowed())
		{
			bInitializeRayTracingRepresentation = false;
		}
#endif

		if(bInitializeRayTracingRepresentation)
		{
			GetRenderData()->InitializeRayTracingRepresentationFromRenderingLODs();
		}

		InitResources();

		// Set up RenderData bounds and LOD data
		GetRenderData()->Bounds = MeshDescriptions[0]->GetBounds();
		CalculateExtendedBounds();

		for (int32 LOD = 0; LOD < NewNumLODs; ++LOD)
		{
			// @todo: some way of customizing LOD screen size and/or calculate it based on mesh bounds
			#if 1
			{
				const float LODPowerBase = 0.75f;
				GetRenderData()->ScreenSize[LOD].Default = FMath::Pow(LODPowerBase, LOD);
			}
			#else
			{
				// Possible model for flexible LODs
				const float MaxDeviation = 100.0f; // specify
				const float PixelError = UE_SMALL_NUMBER;
				const float ViewDistance = (MaxDeviation * 960.0f) / PixelError;

				// Generate a projection matrix.
				const float HalfFOV = UE_PI * 0.25f;
				const float ScreenWidth = 1920.0f;
				const float ScreenHeight = 1080.0f;
				const FPerspectiveMatrix ProjMatrix(HalfFOV, ScreenWidth, ScreenHeight, 1.0f);

				GetRenderData()->ScreenSize[LOD].Default = ComputeBoundsScreenSize(FVector::ZeroVector, GetRenderData()->Bounds.SphereRadius, FVector(0.0f, 0.0f, ViewDistance + GetRenderData()->Bounds.SphereRadius), ProjMatrix);
			}
			#endif
		}

		// Set up physics-related data
		CreateBodySetup();
		check(GetBodySetup());
		GetBodySetup()->InvalidatePhysicsData();

		if (Params.bBuildSimpleCollision)
		{
			FKBoxElem BoxElem;
			BoxElem.Center = GetRenderData()->Bounds.Origin;
			BoxElem.X = GetRenderData()->Bounds.BoxExtent.X * 2.0f;
			BoxElem.Y = GetRenderData()->Bounds.BoxExtent.Y * 2.0f;
			BoxElem.Z = GetRenderData()->Bounds.BoxExtent.Z * 2.0f;
			GetBodySetup()->AggGeom.BoxElems.Add(BoxElem);
			GetBodySetup()->CreatePhysicsMeshes();
		}
	}
#if WITH_EDITOR
	else
	{
		Build(true);
	}

	for (int32 LODIndex = 0; LODIndex < NewNumLODs; LODIndex++)
	{
		FStaticMeshLODResources& LODResources = GetRenderData()->LODResources[LODIndex];
		for (int32 SectionIndex = 0; SectionIndex < LODResources.Sections.Num(); SectionIndex++)
		{
			const FStaticMeshSection& StaticMeshSection = LODResources.Sections[SectionIndex];
			FMeshSectionInfo SectionInfo;
			SectionInfo.MaterialIndex = StaticMeshSection.MaterialIndex;
			SectionInfo.bEnableCollision = StaticMeshSection.bEnableCollision;
			SectionInfo.bCastShadow = StaticMeshSection.bCastShadow;
			GetSectionInfoMap().Set(LODIndex, SectionIndex, SectionInfo);
		}
	}
#endif

	if (!bNewMesh)
	{
		for (FThreadSafeObjectIterator Iter(UStaticMeshComponent::StaticClass()); Iter; ++Iter)
		{
			UStaticMeshComponent* StaticMeshComponent = Cast<UStaticMeshComponent>(*Iter);
			if (StaticMeshComponent->GetStaticMesh() == this)
			{
				// it needs to recreate IF it already has been created
				if (StaticMeshComponent->IsPhysicsStateCreated())
				{
					StaticMeshComponent->RecreatePhysicsState();
				}
			}
		}
	}

	return true;
}

bool UStaticMesh::CanBeClusterRoot() const
{
	return false;
}

int32 UStaticMesh::CalcCumulativeLODSize(int32 NumLODs) const
{
	uint32 Accum = 0;
	const int32 LODCount = GetNumLODs();
	const int32 LastLODIdx = LODCount - NumLODs;
	for (int32 Idx = LODCount - 1; Idx >= LastLODIdx; --Idx)
	{
		Accum += GetRenderData()->LODResources[Idx].BuffersSize;
	}
	return Accum;
}

FIoFilenameHash UStaticMesh::GetMipIoFilenameHash(const int32 MipIndex) const
{
	if (GetRenderData() && GetRenderData()->LODResources.IsValidIndex(MipIndex))
	{
		return GetRenderData()->LODResources[MipIndex].StreamingBulkData.GetIoFilenameHash();
	}
	else
	{
		return INVALID_IO_FILENAME_HASH;
	}
}

bool UStaticMesh::DoesMipDataExist(const int32 MipIndex) const
{
	return GetRenderData() && GetRenderData()->LODResources.IsValidIndex(MipIndex) && GetRenderData()->LODResources[MipIndex].StreamingBulkData.DoesExist();
}

bool UStaticMesh::HasPendingRenderResourceInitialization() const
{
	// Verify we're not compiling before accessing the renderdata to avoid forcing the compilation
	// to finish during garbage collection. If we're still compiling, the render data has not
	// yet been created, hence it is not possible we're actively streaming anything from it...

	// Only check !bReadyForStreaming if the render data is initialized from FStaticMeshRenderData::InitResources(), 
	// otherwise no render commands are pending and the state will never resolve.
	// Note that bReadyForStreaming is set on the renderthread.
	return !IsCompiling() && GetRenderData() && GetRenderData()->IsInitialized() && !GetRenderData()->bReadyForStreaming;
}

bool UStaticMesh::StreamOut(int32 NewMipCount)
{
	check(IsInGameThread());
	if (!HasPendingInitOrStreaming() && CachedSRRState.StreamOut(NewMipCount))
	{
		// We need to keep the CPU data in non cook in order to be able for tools to work correctly.
		PendingUpdate = new FStaticMeshStreamOut(this, bAllowCPUAccess && !FPlatformProperties::HasEditorOnlyData());
		return !PendingUpdate->IsCancelled();
	}
	return false;
}

bool UStaticMesh::StreamIn(int32 NewMipCount, bool bHighPrio)
{
	check(IsInGameThread());
	if (!HasPendingInitOrStreaming() && CachedSRRState.StreamIn(NewMipCount))
	{
		FRenderAssetUpdate::EThreadType CreateResourcesThread = GRHISupportsAsyncTextureCreation
			? FRenderAssetUpdate::TT_Async
			: FRenderAssetUpdate::TT_Render;

#if WITH_EDITOR
		if (FPlatformProperties::HasEditorOnlyData())
		{
			PendingUpdate = new FStaticMeshStreamIn_DDC(this, CreateResourcesThread);
		}
		else
#endif
		{
			PendingUpdate = new FStaticMeshStreamIn_IO(this, bHighPrio, CreateResourcesThread);
		}
		return !PendingUpdate->IsCancelled();
	}
	return false;
}

EStreamableRenderAssetType UStaticMesh::GetRenderAssetType() const
{
	return EStreamableRenderAssetType::StaticMesh;
}

void UStaticMesh::CancelAllPendingStreamingActions()
{
	FlushRenderingCommands();

	for (TObjectIterator<UStaticMesh> It; It; ++It)
	{
		UStaticMesh* StaticMesh = *It;
		StaticMesh->CancelPendingStreamingRequest();
	}

	FlushRenderingCommands();
}

//
//	UStaticMesh::GetDesc
//

/** 
 * Returns a one line description of an object for viewing in the thumbnail view of the generic browser
 */
FString UStaticMesh::GetDesc()
{
	int32 NumTris = 0;
	int32 NumVerts = 0;
	int32 NumLODs = GetRenderData() ? GetRenderData()->LODResources.Num() : 0;
	if (NumLODs > 0)
	{
		NumTris = GetRenderData()->LODResources[0].GetNumTriangles();
		NumVerts = GetRenderData()->LODResources[0].GetNumVertices();
	}
	return FString::Printf(
		TEXT("%d LODs, %d Tris, %d Verts"),
		NumLODs,
		NumTris,
		NumVerts
		);
}


static int32 GetCollisionVertIndexForMeshVertIndex(int32 MeshVertIndex, TMap<int32, int32>& MeshToCollisionVertMap, TArray<FVector3f>& OutPositions, TArray< TArray<FVector2D> >& OutUVs, FPositionVertexBuffer& InPosVertBuffer, FStaticMeshVertexBuffer& InVertBuffer)
{
	int32* CollisionIndexPtr = MeshToCollisionVertMap.Find(MeshVertIndex);
	if (CollisionIndexPtr != nullptr)
	{
		return *CollisionIndexPtr;
	}
	else
	{
		// Copy UVs for vert if desired
		for (int32 ChannelIdx = 0; ChannelIdx < OutUVs.Num(); ChannelIdx++)
		{
			check(OutPositions.Num() == OutUVs[ChannelIdx].Num());
			OutUVs[ChannelIdx].Add(FVector2D(InVertBuffer.GetVertexUV(MeshVertIndex, ChannelIdx)));
		}

		// Copy position
		int32 CollisionVertIndex = OutPositions.Add(InPosVertBuffer.VertexPosition(MeshVertIndex));

		// Add indices to map
		MeshToCollisionVertMap.Add(MeshVertIndex, CollisionVertIndex);

		return CollisionVertIndex;
	}
}

bool UStaticMesh::GetPhysicsTriMeshData(struct FTriMeshCollisionData* CollisionData, bool bInUseAllTriData)
{
	bool bInCheckComplexCollisionMesh = true;
	return GetPhysicsTriMeshDataCheckComplex(CollisionData, bInUseAllTriData, bInCheckComplexCollisionMesh);
}

bool UStaticMesh::GetPhysicsTriMeshDataCheckComplex(struct FTriMeshCollisionData* CollisionData, bool bInUseAllTriData, bool bInCheckComplexCollisionMesh)
{
#if WITH_EDITORONLY_DATA
	if (ComplexCollisionMesh && ComplexCollisionMesh != this && bInCheckComplexCollisionMesh)
	{
		return ComplexCollisionMesh->GetPhysicsTriMeshDataCheckComplex(CollisionData, bInUseAllTriData, false); // Only one level of recursion
	}
#else // #if WITH_EDITORONLY_DATA
	// the static mesh needs to be tagged for CPUAccess in order to access TriMeshData in runtime mode
	// we must also check that the selected LOD has CPU data (see below)
	if (!bAllowCPUAccess)
	{
		UE_LOGF(LogStaticMesh, Warning, "UStaticMesh::GetPhysicsTriMeshData: Triangle data from '%ls' cannot be accessed at runtime on a mesh that isn't flagged as Allow CPU Access. This asset needs to be flagged as such (in the Advanced section).", *GetFullName());
		return false;
	}
#endif // #if !WITH_EDITORONLY_DATA

#if WITH_EDITORONLY_DATA
	// if we're a cooked cooker, just use the canned data
	if (UNLIKELY(GetRenderData() && GetRenderData()->CollisionDataForCookedCooker))
	{
		*CollisionData = *GetRenderData()->CollisionDataForCookedCooker;
	}
	else
	{
#endif // #if !WITH_EDITORONLY_DATA

	check(HasValidRenderData());

	// Get the LOD level to use for collision
	// Always use 0 if asking for 'all tri data'
	const int32 UseLODIndex = bInUseAllTriData ? 0 : FMath::Clamp(LODForCollision, 0, GetRenderData()->LODResources.Num()-1);

	FStaticMeshLODResources& LOD = GetRenderData()->LODResources[UseLODIndex];

	// Make sure the LOD we selected actually has CPU data
	if (!LOD.IndexBuffer.GetAllowCPUAccess())
	{
		UE_LOGF(LogStaticMesh, Warning, "UStaticMesh::GetPhysicsTriMeshData: CPU data not available on selected LOD (UseLODIndex=%d, LODForCollision=%d) on '%ls'.", UseLODIndex, LODForCollision, *GetFullName());
		return false;
	}

	FIndexArrayView Indices = LOD.IndexBuffer.GetArrayView();

	TMap<int32, int32> MeshToCollisionVertMap; // map of static mesh verts to collision verts

	bool bCopyUVs = bSupportPhysicalMaterialMasks || UPhysicsSettings::Get()->bSupportUVFromHitResults; // See if we should copy UVs

	// If copying UVs, allocate array for storing them
	if (bCopyUVs)
	{
		CollisionData->UVs.AddZeroed(LOD.GetNumTexCoords());
	}

	for(int32 SectionIndex = 0; SectionIndex < LOD.Sections.Num(); ++SectionIndex)
	{
		const FStaticMeshSection& Section = LOD.Sections[SectionIndex];

		if (bInUseAllTriData || SectionHasCollisionEnabled(Section, UseLODIndex, SectionIndex))
		{
			const uint32 OnePastLastIndex  = Section.FirstIndex + Section.NumTriangles*3;

			for (uint32 TriIdx = Section.FirstIndex; TriIdx < OnePastLastIndex; TriIdx += 3)
			{
				if ((TriIdx + 2) >= static_cast<uint32>(Indices.Num()))
				{
					UE_LOGF(LogStaticMesh, Error, "UStaticMesh::GetPhysicsTriMeshData: Triangle data from '%ls' is unexpectedly missing.", *GetFullName());
					return false;
				}
				FTriIndices TriIndex;
				TriIndex.v0 = GetCollisionVertIndexForMeshVertIndex(Indices[TriIdx +0], MeshToCollisionVertMap, CollisionData->Vertices, CollisionData->UVs, LOD.VertexBuffers.PositionVertexBuffer, LOD.VertexBuffers.StaticMeshVertexBuffer);
				TriIndex.v1 = GetCollisionVertIndexForMeshVertIndex(Indices[TriIdx +1], MeshToCollisionVertMap, CollisionData->Vertices, CollisionData->UVs, LOD.VertexBuffers.PositionVertexBuffer, LOD.VertexBuffers.StaticMeshVertexBuffer);
				TriIndex.v2 = GetCollisionVertIndexForMeshVertIndex(Indices[TriIdx +2], MeshToCollisionVertMap, CollisionData->Vertices, CollisionData->UVs, LOD.VertexBuffers.PositionVertexBuffer, LOD.VertexBuffers.StaticMeshVertexBuffer);

				CollisionData->Indices.Add(TriIndex);
				CollisionData->MaterialIndices.Add(Section.MaterialIndex);
			}
		}
	}
	CollisionData->bFlipNormals = true;

#if WITH_EDITORONLY_DATA
	}
#endif
	// We only have a valid TriMesh if the CollisionData has vertices AND indices. For meshes with disabled section collision, it
	// can happen that the indices will be empty, in which case we do not want to consider that as valid trimesh data
	return CollisionData->Vertices.Num() > 0 && CollisionData->Indices.Num() > 0;
}

bool UStaticMesh::PollAsyncPhysicsTriMeshDataCheckComplex(bool InUseAllTriData, bool bInCheckComplexCollisionMesh) const
{
#if WITH_EDITORONLY_DATA
	if(ComplexCollisionMesh && ComplexCollisionMesh != this && bInCheckComplexCollisionMesh)
	{
		// Match GetPhysicsTriMeshDataCheck recursion limits on complex mesh to avoid loops
		return ComplexCollisionMesh->PollAsyncPhysicsTriMeshDataCheckComplex(InUseAllTriData, false);
	}
#endif
	return !IsCompiling();
}

bool UStaticMesh::ContainsPhysicsTriMeshData(bool bInUseAllTriData) const
{
	bool bInCheckComplexCollisionMesh = true;
	return ContainsPhysicsTriMeshDataCheckComplex(bInUseAllTriData, bInCheckComplexCollisionMesh);
}

bool UStaticMesh::GetTriMeshSizeEstimates(struct FTriMeshCollisionDataEstimates& OutTriMeshEstimates, bool bInUseAllTriData) const
{
#if WITH_EDITORONLY_DATA
	if (ComplexCollisionMesh && ComplexCollisionMesh != this)
	{
		ComplexCollisionMesh->ConditionalPostLoad();
		return ComplexCollisionMesh->GetTriMeshSizeEstimates(OutTriMeshEstimates, bInUseAllTriData);
	}
#endif // #if WITH_EDITORONLY_DATA

	if (GetRenderData() == nullptr || GetRenderData()->LODResources.Num() == 0)
	{
		return false;
	}

	// Get the LOD level to use for collision
	// Always use 0 if asking for 'all tri data'
	const int32 UseLODIndex = bInUseAllTriData ? 0 : FMath::Clamp(LODForCollision, 0, GetRenderData()->LODResources.Num() - 1);

	const FStaticMeshLODResources& LOD = GetRenderData()->LODResources[UseLODIndex];

	OutTriMeshEstimates.VerticeCount = 0;

	for (int32 SectionIndex = 0; SectionIndex < LOD.Sections.Num(); ++SectionIndex)
	{
		const FStaticMeshSection& Section = LOD.Sections[SectionIndex];

		if (bInUseAllTriData || SectionHasCollisionEnabled(Section, UseLODIndex, SectionIndex))
		{
			OutTriMeshEstimates.VerticeCount += Section.NumTriangles * 3;
		}
	}

	return true;
}

bool UStaticMesh::ContainsPhysicsTriMeshDataCheckComplex(bool bInUseAllTriData, bool bInCheckComplexCollisionMesh) const
{
#if WITH_EDITORONLY_DATA
	if (ComplexCollisionMesh && ComplexCollisionMesh != this && bInCheckComplexCollisionMesh)
	{
		ComplexCollisionMesh->ConditionalPostLoad();
		return ComplexCollisionMesh->ContainsPhysicsTriMeshDataCheckComplex(bInUseAllTriData, false); // One level of recursion
	}
#endif // #if WITH_EDITORONLY_DATA
	
#if WITH_EDITORONLY_DATA
	// if we're a cooked cooker, just use the canned data
	if (UNLIKELY(GetRenderData() && GetRenderData()->CollisionDataForCookedCooker))
	{
		return !GetRenderData()->CollisionDataForCookedCooker->Vertices.IsEmpty();
	}
#endif // #if !WITH_EDITORONLY_DATA

	if(GetRenderData() == nullptr || GetRenderData()->LODResources.Num() == 0)
	{
		return false;
	}

	// Get the LOD level to use for collision
	// Always use 0 if asking for 'all tri data'
	const int32 UseLODIndex = bInUseAllTriData ? 0 : FMath::Clamp(LODForCollision, 0, GetRenderData()->LODResources.Num() - 1);

	const FStaticMeshLODResources& LOD = GetRenderData()->LODResources[UseLODIndex];
	if (LOD.VertexBuffers.PositionVertexBuffer.GetNumVertices() > 0)
	{
		for (int32 SectionIndex = 0; SectionIndex < LOD.Sections.Num(); ++SectionIndex)
		{
			const FStaticMeshSection& Section = LOD.Sections[SectionIndex];
			if (Section.NumTriangles == 0)
			{
				continue;
			}

			if (bInUseAllTriData || SectionHasCollisionEnabled(Section, UseLODIndex, SectionIndex))
			{
				return true;
			}
		}
	}
	return false; 
}

bool UStaticMesh::SectionHasCollisionEnabled(const FStaticMeshSection& Section, int32 LODIndex, int32 SectionIndex) const
{
#if WITH_EDITORONLY_DATA
	// we can only use GetSectionInfoMap() in WITH_EDITORONLY_DATA mode
	return GetSectionInfoMap().Get(LODIndex, SectionIndex).bEnableCollision;
#else // #if WITH_EDITORONLY_DATA
	return Section.bEnableCollision;
#endif // #if !WITH_EDITORONLY_DATA
}

bool UStaticMesh::PollAsyncPhysicsTriMeshData(bool InUseAllTriData) const
{
	bool bInCheckComplexCollisionMesh = true;
	return PollAsyncPhysicsTriMeshDataCheckComplex(InUseAllTriData, bInCheckComplexCollisionMesh);
}

void UStaticMesh::GetMeshId(FString& OutMeshId)
{
#if WITH_EDITORONLY_DATA
	OutMeshId.Reset();
	if (ComplexCollisionMesh && ComplexCollisionMesh->GetRenderData())
	{
		OutMeshId = ComplexCollisionMesh->GetRenderData()->DerivedDataKey;
	}
	if (GetRenderData())
	{
		OutMeshId.Append(GetRenderData()->DerivedDataKey);
	}
#endif
}

void UStaticMesh::AddAssetUserData(UAssetUserData* InUserData)
{
	if(InUserData != NULL)
	{
		UAssetUserData* ExistingData = GetAssetUserDataOfClass(InUserData->GetClass());
		if(ExistingData != NULL)
		{
			AssetUserData.Remove(ExistingData);
		}
		AssetUserData.Add(InUserData);
	}
}

UAssetUserData* UStaticMesh::GetAssetUserDataOfClass(TSubclassOf<UAssetUserData> InUserDataClass)
{
	for(int32 DataIdx=0; DataIdx<AssetUserData.Num(); DataIdx++)
	{
		UAssetUserData* Datum = AssetUserData[DataIdx];
		if(Datum != NULL && Datum->IsA(InUserDataClass))
		{
			return Datum;
		}
	}
	return NULL;
}

void UStaticMesh::RemoveUserDataOfClass(TSubclassOf<UAssetUserData> InUserDataClass)
{
	for(int32 DataIdx=0; DataIdx<AssetUserData.Num(); DataIdx++)
	{
		UAssetUserData* Datum = AssetUserData[DataIdx];
		if(Datum != NULL && Datum->IsA(InUserDataClass))
		{
			AssetUserData.RemoveAt(DataIdx);
			return;
		}
	}
}

const TArray<UAssetUserData*>* UStaticMesh::GetAssetUserDataArray() const 
{
	return &ToRawPtrTArrayUnsafe(AssetUserData);
}

/**
 * Create BodySetup for this staticmesh 
 */
void UStaticMesh::CreateBodySetup()
{
	if (GetBodySetup() == nullptr)
	{
		UBodySetup* NewBodySetup = nullptr;
		{
			FGCScopeGuard Scope;
			NewBodySetup = NewObject<UBodySetup>(this);
		}
		NewBodySetup->DefaultInstance.SetCollisionProfileName(UCollisionProfile::BlockAll_ProfileName);
		NewBodySetup->bSupportUVsAndFaceRemap = bSupportPhysicalMaterialMasks;
		SetBodySetup(NewBodySetup);
	}
}

void UStaticMesh::CreateNavCollision(const bool bIsUpdate)
{
	if (bHasNavigationData && GetBodySetup() != nullptr)
	{
		if (GetNavCollision() == nullptr)
		{
			UNavCollisionBase* NewNavCollisionBase = nullptr;
			{
				FGCScopeGuard Scope;
				NewNavCollisionBase = UNavCollisionBase::ConstructNew(*this);
			}
			SetNavCollision(NewNavCollisionBase);
		}

		if (GetNavCollision())
		{
#if WITH_EDITOR
			if (bIsUpdate)
			{
				GetNavCollision()->InvalidateCollision();
			}
#endif // WITH_EDITOR

			// Physics meshes need to be ready to gather the collision in Setup().
			GetBodySetup()->CreatePhysicsMeshes();
			GetNavCollision()->Setup(GetBodySetup());
		}
	}
	else
	{
		SetNavCollision(nullptr);
	}
}

void UStaticMesh::RecreateNavCollision()
{
	SetNavCollision(nullptr);
	CreateNavCollision();
}

void UStaticMesh::SetNavCollision(UNavCollisionBase* InNavCollision)
{
	WaitUntilAsyncPropertyReleased(EStaticMeshAsyncProperties::NavCollision);
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	NavCollision = InNavCollision;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

UNavCollisionBase* UStaticMesh::GetNavCollision() const
{
	WaitUntilAsyncPropertyReleased(EStaticMeshAsyncProperties::NavCollision);
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	return NavCollision; 
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

FBox UStaticMesh::GetNavigationBounds(const FTransform& LocalToWorld) const
{
	FBox NavBounds = GetBounds().GetBox();
	if (const UNavCollisionBase* NavCol = GetNavCollision())
	{
		const FBox NavCollisionBounds = NavCol->GetBounds();
		if (NavCollisionBounds.IsValid)
		{
			NavBounds = NavCollisionBounds;
		}
	}
	return NavBounds.TransformBy(LocalToWorld);
}

void UStaticMesh::MarkAsNotHavingNavigationData()
{
	bHasNavigationData = false;
	SetNavCollision(nullptr);
}

/**
 * Returns vertex color data by position.
 * For matching to reimported meshes that may have changed or copying vertex paint data from mesh to mesh.
 *
 *	@param	VertexColorData		(out)A map of vertex position data and its color. The method fills this map.
 */
void UStaticMesh::GetVertexColorData(TMap<FVector3f, FColor>& VertexColorData)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UStaticMesh::GetVertexColorData);
	VertexColorData.Empty();
#if WITH_EDITOR
	// What LOD to get vertex colors from.  
	// Currently mesh painting only allows for painting on the first lod.
	const uint32 PaintingMeshLODIndex = 0;
	if (IsSourceModelValid(PaintingMeshLODIndex))
	{
		if (!GetSourceModel(PaintingMeshLODIndex).IsRawMeshEmpty())
		{
			FMeshDescription* MeshDescription = GetMeshDescription(PaintingMeshLODIndex);
			VertexColorData.Reserve(MeshDescription->Vertices().Num());
			TVertexAttributesConstRef<FVector3f> Positions = FStaticMeshConstAttributes(*MeshDescription).GetVertexPositions();
			TVertexInstanceAttributesConstRef<FVector4f> Colors = FStaticMeshConstAttributes(*MeshDescription).GetVertexInstanceColors();
			for(FVertexInstanceID VertexInstanceID : MeshDescription->VertexInstances().GetElementIDs())
			{
				FVertexID VertexID = MeshDescription->GetVertexInstanceVertex(VertexInstanceID);
				FColor Color = FLinearColor(Colors[VertexInstanceID]).ToFColorSRGB();
				const FVector3f& Position = Positions[VertexID];
				if (!VertexColorData.Contains(Position))
				{
					VertexColorData.Add(Position, Color);
				}
			}
		}
	}
#endif // #if WITH_EDITORONLY_DATA
}

/**
 * Sets vertex color data by position.
 * Map of vertex color data by position is matched to the vertex position in the mesh
 * and nearest matching vertex color is used.
 *
 *	@param	VertexColorData		A map of vertex position data and color.
 */
void UStaticMesh::SetVertexColorData(const TMap<FVector3f, FColor>& VertexColorData)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UStaticMesh::SetVertexColorData);
#if WITH_EDITOR
	// What LOD to get vertex colors from.  
	// Currently mesh painting only allows for painting on the first lod.
	const uint32 PaintingMeshLODIndex = 0;
	if (IsSourceModelValid(PaintingMeshLODIndex))
	{
		if (GetSourceModel(PaintingMeshLODIndex).IsRawMeshEmpty() == false)
		{
			FMeshDescription* MeshDescription = GetMeshDescription(PaintingMeshLODIndex);
			TVertexAttributesRef<FVector3f> Positions = FStaticMeshAttributes(*MeshDescription).GetVertexPositions();
			TVertexInstanceAttributesRef<FVector4f> Colors = FStaticMeshAttributes(*MeshDescription).GetVertexInstanceColors();
			for (FVertexInstanceID VertexInstanceID : MeshDescription->VertexInstances().GetElementIDs())
			{
				FVertexID VertexID = MeshDescription->GetVertexInstanceVertex(VertexInstanceID);
				const FVector3f& Position = Positions[VertexID];
				if (const FColor* Color = VertexColorData.Find(Position))
				{
					Colors[VertexInstanceID] = FVector4f(FLinearColor::FromSRGBColor(*Color));
				}
				else
				{
					Colors[VertexInstanceID] = FVector4f(FLinearColor::White);
				}
			}
			CommitMeshDescription(PaintingMeshLODIndex);
		}
	}
	// TODO_STATICMESH: Build?
#endif // #if WITH_EDITOR
}

ENGINE_API void UStaticMesh::RemoveVertexColors()
{
#if WITH_EDITOR
	bool bRemovedVertexColors = false;

	for (int32 LodIndex = 0; LodIndex < GetNumSourceModels(); LodIndex++)
	{
		FStaticMeshSourceModel& SourceModel = GetSourceModel(LodIndex);
		if (!SourceModel.IsRawMeshEmpty())
		{
			FMeshDescription* MeshDescription = GetMeshDescription(LodIndex);
			TVertexInstanceAttributesRef<FVector4f> Colors = FStaticMeshAttributes(*MeshDescription).GetVertexInstanceColors();
			for (FVertexInstanceID VertexInstanceID : MeshDescription->VertexInstances().GetElementIDs())
			{
				Colors[VertexInstanceID] = FVector4f(FLinearColor::White);
			}
			CommitMeshDescription(LodIndex);
			bRemovedVertexColors = true;
		}
	}

	if (bRemovedVertexColors)
	{
		Build();
		MarkPackageDirty();
		OnMeshChanged.Broadcast();
	}
#endif
}

void UStaticMesh::EnforceLightmapRestrictions(bool bUseRenderData)
{
	// Legacy content may contain a lightmap resolution of 0, which was valid when vertex lightmaps were supported, but not anymore with only texture lightmaps
	SetLightMapResolution(FMath::Max(GetLightMapResolution(), 4));

	// Lightmass only supports 4 UVs from Lightmass::MAX_TEXCOORDS
	int32 NumUVs = 4;

#if !WITH_EDITORONLY_DATA
	if (!bUseRenderData)
	{
		//The source models are only available in the editor, fallback on the render data.
		UE_ASSET_LOG(LogStaticMesh, Warning, this, TEXT("Trying to enforce lightmap restrictions using the static mesh SourceModels outside of the Editor."))
		bUseRenderData = true;
	}
#endif //WITH_EDITORONLY_DATA

	if (bUseRenderData)
	{
		if (GetRenderData())
		{
			for (int32 LODIndex = 0; LODIndex < GetRenderData()->LODResources.Num(); ++LODIndex)
			{
				const FStaticMeshLODResources& LODResource = GetRenderData()->LODResources[LODIndex];
				if (LODResource.GetNumVertices() > 0) // skip LOD that was stripped (eg. MinLOD)
				{
					NumUVs = FMath::Min(LODResource.GetNumTexCoords(), NumUVs);
				}
			}
		}
		else
		{
			NumUVs = 1;
		}
	}
#if WITH_EDITORONLY_DATA
	else
	{
		for (int32 LODIndex = 0; LODIndex < GetNumSourceModels(); ++LODIndex)
		{
			//If the LOD is generated we validate the BaseLODModel instead as the generated LOD is not available before the build and has the same UV properties as its base LOD.
			const bool bIsGeneratedLOD = !IsMeshDescriptionValid(LODIndex) || IsReductionActive(LODIndex);
			const int32 SourceLOD = bIsGeneratedLOD ? GetSourceModel(LODIndex).ReductionSettings.BaseLODModel : LODIndex;

			if (!bIsGeneratedLOD || LODIndex == SourceLOD)
			{
				if (const FMeshDescription* MeshDescription = GetMeshDescription(SourceLOD))
				{
					TVertexInstanceAttributesConstRef<FVector2f> UVChannels = FStaticMeshConstAttributes(*MeshDescription).GetVertexInstanceUVs();

					// skip empty/stripped LODs
					if (UVChannels.GetNumElements() > 0)
					{
						int NumChannelsInLOD = UVChannels.GetNumChannels();
						const FStaticMeshSourceModel& SourceModel = GetSourceModel(SourceLOD);

						if (SourceModel.BuildSettings.bGenerateLightmapUVs)
						{
							NumChannelsInLOD = FMath::Max(NumChannelsInLOD, SourceModel.BuildSettings.DstLightmapIndex + 1);
						}

						NumUVs = FMath::Min(NumChannelsInLOD, NumUVs);
					}
				}
				else
				{
					NumUVs = 1;
					break;
				}
			}
		}

		if (GetNumSourceModels() == 0)
		{
			NumUVs = 1;
		}
	}
#endif //WITH_EDITORONLY_DATA

	// do not allow LightMapCoordinateIndex go negative
	check(NumUVs > 0);

	// Clamp LightMapCoordinateIndex to be valid for all lightmap uvs
	SetLightMapCoordinateIndex(FMath::Clamp(GetLightMapCoordinateIndex(), 0, NumUVs - 1));
}

/**
 * Static: Processes the specified static mesh for light map UV problems
 *
 * @param	InStaticMesh					Static mesh to process
 * @param	InOutAssetsWithMissingUVSets	Array of assets that we found with missing UV sets
 * @param	InOutAssetsWithBadUVSets		Array of assets that we found with bad UV sets
 * @param	InOutAssetsWithValidUVSets		Array of assets that we found with valid UV sets
 * @param	bInVerbose						If true, log the items as they are found
 */
void UStaticMesh::CheckLightMapUVs( UStaticMesh* InStaticMesh, TArray< FString >& InOutAssetsWithMissingUVSets, TArray< FString >& InOutAssetsWithBadUVSets, TArray< FString >& InOutAssetsWithValidUVSets, bool bInVerbose )
{
	if (!IsStaticLightingAllowed())
	{
		// We do not need to check for lightmap UV problems when we do not allow static lighting
		return;
	}

	struct FLocal
	{
		/**
		 * Checks to see if a point overlaps a triangle
		 *
		 * @param	P	Point
		 * @param	A	First triangle vertex
		 * @param	B	Second triangle vertex
		 * @param	C	Third triangle vertex
		 *
		 * @return	true if the point overlaps the triangle
		 */
		bool IsPointInTriangle( const FVector& P, const FVector& A, const FVector& B, const FVector& C, const float Epsilon )
		{
			struct
			{
				bool SameSide( const FVector& P1, const FVector& P2, const FVector& InA, const FVector& InB, const float InEpsilon )
				{
					const FVector Cross1((InB - InA) ^ (P1 - InA));
					const FVector Cross2((InB - InA) ^ (P2 - InA));
					return (Cross1 | Cross2) >= -InEpsilon;
				}
			} Local;

			return ( Local.SameSide( P, A, B, C, Epsilon ) &&
					 Local.SameSide( P, B, A, C, Epsilon ) &&
					 Local.SameSide( P, C, A, B, Epsilon ) );
		}
		
		/**
		 * Checks to see if a point overlaps a triangle
		 *
		 * @param	P	Point
		 * @param	Triangle	triangle vertices
		 *
		 * @return	true if the point overlaps the triangle
		 */
		bool IsPointInTriangle(const FVector2D & P, const FVector2D (&Triangle)[3])
		{
			// Bias toward non-overlapping so sliver triangles won't overlap their adjoined neighbors
			const float TestEpsilon = -0.001f;
			// Test for overlap
			if( IsPointInTriangle(
				FVector( P, 0.0f ),
				FVector( Triangle[0], 0.0f ),
				FVector( Triangle[1], 0.0f ),
				FVector( Triangle[2], 0.0f ),
				TestEpsilon ) )
			{
				return true;
			}
			return false;
		}

		/**
		 * Checks for UVs outside of a 0.0 to 1.0 range.
		 *
		 * @param	TriangleUVs	a referenced array of 3 UV coordinates.
		 *
		 * @return	true if UVs are <0.0 or >1.0
		 */
		bool AreUVsOutOfRange(const FVector2D (&TriangleUVs)[3])
		{
			// Test for UVs outside of the 0.0 to 1.0 range (wrapped/clamped)
			for(int32 UVIndex = 0; UVIndex < 3; UVIndex++)
			{
				const FVector2D& CurVertUV = TriangleUVs[UVIndex];
				const float TestEpsilon = 0.001f;
				for( int32 CurDimIndex = 0; CurDimIndex < 2; ++CurDimIndex )
				{
					if( CurVertUV[ CurDimIndex ] < ( 0.0f - TestEpsilon ) || CurVertUV[ CurDimIndex ] > ( 1.0f + TestEpsilon ) )
					{
						return true;
					}
				}
			}
			return false;
		}

		/**
		 * Fills an array with 3 UV coordinates for a specified triangle from a FStaticMeshLODResources object.
		 *
		 * @param	MeshLOD	Source mesh.
		 * @param	TriangleIndex	triangle to get UV data from
		 * @param	UVChannel UV channel to extract
		 * @param	TriangleUVsOUT an array which is filled with the UV data
		 */
		void GetTriangleUVs( const FStaticMeshLODResources& MeshLOD, const int32 TriangleIndex, const int32 UVChannel, FVector2D (&TriangleUVsOUT)[3])
		{
			check( TriangleIndex < MeshLOD.GetNumTriangles());
			
			FIndexArrayView Indices = MeshLOD.IndexBuffer.GetArrayView();
			const int32 StartIndex = TriangleIndex*3;			
			const uint32 VertexIndices[] = {Indices[StartIndex + 0], Indices[StartIndex + 1], Indices[StartIndex + 2]};
			for(int i = 0; i<3;i++)
			{
				TriangleUVsOUT[i] = FVector2D(MeshLOD.VertexBuffers.StaticMeshVertexBuffer.GetVertexUV(VertexIndices[i], UVChannel));		
			}
		}

		enum UVCheckResult { UVCheck_Missing, UVCheck_Bad, UVCheck_OK, UVCheck_NoTriangles};
		/**
		 * Performs a UV check on a specific LOD from a UStaticMesh.
		 *
		 * @param	MeshLOD	a referenced array of 3 UV coordinates.
		 * @param	LightMapCoordinateIndex The UV channel containing the light map UVs.
		 * @param	OverlappingLightMapUVTriangleCountOUT Filled with the number of triangles that overlap one another.
		 * @param	OutOfBoundsTriangleCountOUT Filled with the number of triangles whose UVs are out of 0..1 range.
		 * @return	UVCheckResult UVCheck_Missing: light map UV channel does not exist in the data. UVCheck_Bad: one or more triangles break UV mapping rules. UVCheck_NoTriangle: The specified mesh has no triangles. UVCheck_OK: no problems were found.
		 */
		UVCheckResult CheckLODLightMapUVs( const FStaticMeshLODResources& MeshLOD, const int32 InLightMapCoordinateIndex, int32& OverlappingLightMapUVTriangleCountOUT, int32& OutOfBoundsTriangleCountOUT)
		{
			const int32 TriangleCount = MeshLOD.GetNumTriangles();
			if(TriangleCount==0)
			{
				return UVCheck_NoTriangles;
			}
			OverlappingLightMapUVTriangleCountOUT = 0;
			OutOfBoundsTriangleCountOUT = 0;

			TArray< int32 > TriangleOverlapCounts;
			TriangleOverlapCounts.AddZeroed( TriangleCount );

			if (InLightMapCoordinateIndex >= MeshLOD.GetNumTexCoords())
			{
				return UVCheck_Missing;
			}

			for(int32 CurTri = 0; CurTri<TriangleCount;CurTri++)
			{
				FVector2D CurTriangleUVs[3];
				GetTriangleUVs(MeshLOD, CurTri, InLightMapCoordinateIndex, CurTriangleUVs);
				FVector2D CurTriangleUVCentroid = ( CurTriangleUVs[0] + CurTriangleUVs[1] + CurTriangleUVs[2] ) / 3.0f;
		
				if( AreUVsOutOfRange(CurTriangleUVs) )
				{
					++OutOfBoundsTriangleCountOUT;
				}

				if(TriangleOverlapCounts[CurTri] != 0)
				{
					continue;
				}
				for(int32 OtherTri = CurTri+1; OtherTri<TriangleCount;OtherTri++)
				{
					if(TriangleOverlapCounts[OtherTri] != 0)
					{
						continue;
					}

					FVector2D OtherTriangleUVs[3];
					GetTriangleUVs(MeshLOD, OtherTri, InLightMapCoordinateIndex, OtherTriangleUVs);
					FVector2D OtherTriangleUVCentroid = ( OtherTriangleUVs[0] + OtherTriangleUVs[1] + OtherTriangleUVs[2] ) / 3.0f;

					bool result1 = IsPointInTriangle(CurTriangleUVCentroid, OtherTriangleUVs );
					bool result2 = IsPointInTriangle(OtherTriangleUVCentroid, CurTriangleUVs );

					if( result1 || result2)
					{
						++OverlappingLightMapUVTriangleCountOUT;
						++TriangleOverlapCounts[ CurTri ];
						++OverlappingLightMapUVTriangleCountOUT;
						++TriangleOverlapCounts[ OtherTri ];
					}
				}
			}

			return (OutOfBoundsTriangleCountOUT != 0 || OverlappingLightMapUVTriangleCountOUT !=0 ) ? UVCheck_Bad : UVCheck_OK;
		}
	} Local;

	check( InStaticMesh != NULL );

	TArray< int32 > TriangleOverlapCounts;

	const int32 NumLods = InStaticMesh->GetNumLODs();
	for( int32 CurLODModelIndex = 0; CurLODModelIndex < NumLods; ++CurLODModelIndex )
	{
		const FStaticMeshLODResources& RenderData = InStaticMesh->GetRenderData()->LODResources[CurLODModelIndex];
		int32 LightMapTextureCoordinateIndex = InStaticMesh->GetLightMapCoordinateIndex();

		// We expect the light map texture coordinate to be greater than zero, as the first UV set
		// should never really be used for light maps, unless this mesh was exported as a light mapped uv set.
		if( LightMapTextureCoordinateIndex <= 0 && RenderData.GetNumTexCoords() > 1 )
		{	
			LightMapTextureCoordinateIndex = 1;
		}

		int32 OverlappingLightMapUVTriangleCount = 0;
		int32 OutOfBoundsTriangleCount = 0;

		const FLocal::UVCheckResult result = Local.CheckLODLightMapUVs( RenderData, LightMapTextureCoordinateIndex, OverlappingLightMapUVTriangleCount, OutOfBoundsTriangleCount);
		switch(result)
		{
			case FLocal::UVCheck_OK:
				InOutAssetsWithValidUVSets.Add( InStaticMesh->GetFullName() );
			break;
			case FLocal::UVCheck_Bad:
				InOutAssetsWithBadUVSets.Add( InStaticMesh->GetFullName() );
			break;
			case FLocal::UVCheck_Missing:
				InOutAssetsWithMissingUVSets.Add( InStaticMesh->GetFullName() );
			break;
			default:
			break;
		}

		if(bInVerbose == true)
		{
			switch(result)
			{
				case FLocal::UVCheck_OK:
					UE_LOGF(LogStaticMesh, Log, "[%ls, LOD %i] light map UVs OK", *InStaticMesh->GetName(), CurLODModelIndex );
					break;
				case FLocal::UVCheck_Bad:
					if( OverlappingLightMapUVTriangleCount > 0 )
					{
						UE_LOGF(LogStaticMesh, Warning, "[%ls, LOD %i] %i triangles with overlapping UVs (of %i) (UV set %i)", *InStaticMesh->GetName(), CurLODModelIndex, OverlappingLightMapUVTriangleCount, RenderData.GetNumTriangles(), LightMapTextureCoordinateIndex );
					}
					if( OutOfBoundsTriangleCount > 0 )
					{
						UE_LOGF(LogStaticMesh, Warning, "[%ls, LOD %i] %i triangles with out-of-bound UVs (of %i) (UV set %i)", *InStaticMesh->GetName(), CurLODModelIndex, OutOfBoundsTriangleCount, RenderData.GetNumTriangles(), LightMapTextureCoordinateIndex );
					}
					break;
				case FLocal::UVCheck_Missing:
					UE_LOGF(LogStaticMesh, Warning, "[%ls, LOD %i] missing light map UVs (Res %i, CoordIndex %i)", *InStaticMesh->GetName(), CurLODModelIndex, InStaticMesh->GetLightMapResolution(), InStaticMesh->GetLightMapCoordinateIndex() );
					break;
				case FLocal::UVCheck_NoTriangles:
					UE_LOGF(LogStaticMesh, Warning, "[%ls, LOD %i] doesn't have any triangles", *InStaticMesh->GetName(), CurLODModelIndex );
					break;
				default:
					break;
			}
		}
	}
}

UMaterialInterface* UStaticMesh::GetMaterial(int32 MaterialIndex) const
{
	if (GetStaticMaterials().IsValidIndex(MaterialIndex))
	{
		return GetStaticMaterials()[MaterialIndex].MaterialInterface;
	}

	return nullptr;
}

void UStaticMesh::GetUsedMaterials(TArray<UMaterialInterface*>& OutMaterials, TFunctionRef<UMaterialInterface*(int32)> OverrideMaterial) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UStaticMesh::GetUsedMaterials);

	if (const FStaticMeshRenderData* ThisRenderData = GetRenderData())
	{
		int32 LastIndex = 0;
		TSet<int32> UniqueIndex;
		for (int32 LODIndex = 0, Num = GetRenderData()->LODResources.Num(); LODIndex < Num; LODIndex++)
		{
			const FStaticMeshLODResources& LODResources = ThisRenderData->LODResources[LODIndex];
			for (int32 SectionIndex = 0; SectionIndex < LODResources.Sections.Num(); SectionIndex++)
			{
				// Get the material for each element at the current lod index
				const int32 MaterialIndex = LODResources.Sections[SectionIndex].MaterialIndex;
				UniqueIndex.Add(MaterialIndex);
				LastIndex = (MaterialIndex < LastIndex) ? -1 : MaterialIndex;
			}
		}

		OutMaterials.Reserve(UniqueIndex.Num());

		if (LastIndex == -1)
		{
			// We need to output the material in the correct order (follow the material index),
			// So we sort the map with the material index
			TArray<int32> SortedUniqueIndex = UniqueIndex.Array();
			SortedUniqueIndex.Sort(TLess<int32>());
			
			for (int32 MaterialIndex : SortedUniqueIndex)
			{
				OutMaterials.Add(OverrideMaterial(MaterialIndex));
			}
		}
		else
		{
			for (int32 MaterialIndex : UniqueIndex)
			{
				OutMaterials.Add(OverrideMaterial(MaterialIndex));
			}
		}
	}
}


FName UStaticMesh::AddMaterial(UMaterialInterface* Material)
{
	if (Material == nullptr)
	{
		return NAME_None;
	}

	// Create a unique slot name for the material
	FName MaterialName = Material->GetFName();
	for (const FStaticMaterial& StaticMaterial : GetStaticMaterials())
	{
		const FName ExistingName = StaticMaterial.MaterialSlotName;
		if (ExistingName.GetComparisonIndex() == MaterialName.GetComparisonIndex())
		{
			MaterialName = FName(MaterialName, FMath::Max(MaterialName.GetNumber(), ExistingName.GetNumber() + 1));
		}
	}

#if WITH_EDITORONLY_DATA
	FStaticMaterial& StaticMaterial = GetStaticMaterials().Emplace_GetRef(Material, MaterialName, MaterialName);
#else
	FStaticMaterial& StaticMaterial = GetStaticMaterials().Emplace_GetRef(Material, MaterialName);
#endif

	StaticMaterial.UVChannelData = FMeshUVChannelInfo(1.0f);

	return MaterialName;
}


int32 UStaticMesh::GetMaterialIndex(FName MaterialSlotName) const
{
	for (int32 MaterialIndex = 0; MaterialIndex < GetStaticMaterials().Num(); ++MaterialIndex)
	{
		const FStaticMaterial &StaticMaterial = GetStaticMaterials()[MaterialIndex];
		if (StaticMaterial.MaterialSlotName == MaterialSlotName)
		{
			return MaterialIndex;
		}
	}
	return INDEX_NONE;
}

#if WITH_EDITOR
void UStaticMesh::SetMaterial(int32 MaterialIndex, UMaterialInterface* NewMaterial)
{
	if (GetStaticMaterials().IsValidIndex(MaterialIndex))
	{
		// Ensure mesh descriptions are loaded before starting the transaction
		for (int32 LODIndex = 0; LODIndex < GetNumSourceModels(); ++LODIndex)
		{
			GetMeshDescription(LODIndex);
		}

		FScopedTransaction ScopeTransaction(LOCTEXT("StaticMeshMaterialChanged", "StaticMesh: Material changed"));

		// flag the property (Materials) we're modifying so that not all of the object is rebuilt.
		FProperty* ChangedProperty = FindFProperty<FProperty>(UStaticMesh::StaticClass(), UStaticMesh::GetStaticMaterialsName());
		check(ChangedProperty);
		PreEditChange(ChangedProperty);
		UMaterialInterface* CancelOldMaterial = GetStaticMaterials()[MaterialIndex].MaterialInterface;
		GetStaticMaterials()[MaterialIndex].MaterialInterface = NewMaterial;
		if (NewMaterial != nullptr)
		{
			//Set the Material slot name to a good default one
			if (GetStaticMaterials()[MaterialIndex].MaterialSlotName == NAME_None)
			{
				GetStaticMaterials()[MaterialIndex].MaterialSlotName = NewMaterial->GetFName();
			}
			
			//Set the original fbx material name so we can re-import correctly, ensure the name is unique
			if (GetStaticMaterials()[MaterialIndex].ImportedMaterialSlotName == NAME_None)
			{
				auto IsMaterialNameUnique = [this, MaterialIndex](const FName TestName)
				{
					for (int32 MatIndex = 0; MatIndex < GetStaticMaterials().Num(); ++MatIndex)
					{
						if (MatIndex == MaterialIndex)
						{
							continue;
						}
						if (GetStaticMaterials()[MatIndex].ImportedMaterialSlotName == TestName)
						{
							return false;
						}
					}
					return true;
				};

				int32 MatchNameCounter = 0;
				//Make sure the name is unique for imported material slot name
				bool bUniqueName = false;
				FString MaterialSlotName = NewMaterial->GetName();
				while (!bUniqueName)
				{
					bUniqueName = true;
					if (!IsMaterialNameUnique(FName(*MaterialSlotName)))
					{
						bUniqueName = false;
						MatchNameCounter++;
						MaterialSlotName = NewMaterial->GetName() + TEXT("_") + FString::FromInt(MatchNameCounter);
					}
				}
				GetStaticMaterials()[MaterialIndex].ImportedMaterialSlotName = FName(*MaterialSlotName);
			}
		}

		if (ChangedProperty)
		{
			FPropertyChangedEvent PropertyUpdateStruct(ChangedProperty);
			PostEditChangeProperty(PropertyUpdateStruct);
		}
		else
		{
			Modify();
			PostEditChange();
		}
		if (GetBodySetup())
		{
			GetBodySetup()->CreatePhysicsMeshes();
		}
	}
}
#endif //WITH_EDITOR

int32 UStaticMesh::GetMaterialIndexFromImportedMaterialSlotName(FName ImportedMaterialSlotName) const
{
	for (int32 MaterialIndex = 0; MaterialIndex < GetStaticMaterials().Num(); ++MaterialIndex)
	{
		const FStaticMaterial &StaticMaterial = GetStaticMaterials()[MaterialIndex];
		if (StaticMaterial.ImportedMaterialSlotName == ImportedMaterialSlotName)
		{
			return MaterialIndex;
		}
	}
	return INDEX_NONE;
}

/**
 * Returns the render data to use for exporting the specified LOD. This method should always
 * be called when exporting a static mesh.
 */
const FStaticMeshLODResources& UStaticMesh::GetLODForExport(int32 LODIndex) const
{
	check(GetRenderData());
	LODIndex = FMath::Clamp<int32>( LODIndex, 0, GetRenderData()->LODResources.Num()-1 );
	// TODO_STATICMESH: Don't allow exporting simplified meshes?
	return GetRenderData()->LODResources[LODIndex];
}

#if WITH_EDITOR
bool UStaticMesh::CanLODsShareStaticLighting() const
{
	bool bCanShareData = true;
	for (int32 LODIndex = 1; bCanShareData && LODIndex < GetNumSourceModels(); ++LODIndex)
	{
		bCanShareData = bCanShareData && !IsMeshDescriptionValid(LODIndex);
	}

	if (SpeedTreeWind.IsValid())
	{
		// SpeedTrees are set up for lighting to share between LODs
		bCanShareData = true;
	}

	return bCanShareData;
}

void UStaticMesh::ConvertLegacyLODDistance()
{
	const int32 NumSourceModels = GetNumSourceModels();
	check(NumSourceModels > 0);
	check(NumSourceModels <= MAX_STATIC_MESH_LODS);

	if(NumSourceModels == 1)
	{
		// Only one model, 
		GetSourceModel(0).ScreenSize.Default = 1.0f;
	}
	else
	{
		// Multiple models, we should have LOD distance data.
		// Assuming an FOV of 90 and a screen size of 1920x1080 to estimate an appropriate display factor.
		const float HalfFOV = UE_PI / 4.0f;
		const float ScreenWidth = 1920.0f;
		const float ScreenHeight = 1080.0f;

		for(int32 ModelIndex = 0 ; ModelIndex < NumSourceModels ; ++ModelIndex)
		{
			FStaticMeshSourceModel& SrcModel = GetSourceModel(ModelIndex);

			if(SrcModel.LODDistance_DEPRECATED == 0.0f)
			{
				SrcModel.ScreenSize.Default = 1.0f;
				GetRenderData()->ScreenSize[ModelIndex] = SrcModel.ScreenSize.Default;
			}
			else
			{
				// Create a screen position from the LOD distance
				const FVector4 PointToTest(0.0f, 0.0f, SrcModel.LODDistance_DEPRECATED, 1.0f);
				FPerspectiveMatrix ProjMatrix(HalfFOV, ScreenWidth, ScreenHeight, 1.0f);
				FVector4 ScreenPosition = ProjMatrix.TransformFVector4(PointToTest);
				// Convert to a percentage of the screen
				const float ScreenMultiple = ScreenWidth / 2.0f * ProjMatrix.M[0][0];
				const float ScreenRadius = ScreenMultiple * GetBounds().SphereRadius / FMath::Max(float(ScreenPosition.W), 1.0f);
				const float ScreenArea = ScreenWidth * ScreenHeight;
				const float BoundsArea = UE_PI * ScreenRadius * ScreenRadius;
				SrcModel.ScreenSize.Default = FMath::Clamp(BoundsArea / ScreenArea, 0.0f, 1.0f);
				GetRenderData()->ScreenSize[ModelIndex] = SrcModel.ScreenSize.Default;
			}
		}
	}
}

void UStaticMesh::ConvertLegacyLODScreenArea()
{
	const int32 NumSourceModels = GetNumSourceModels();
	check(NumSourceModels > 0);
	check(NumSourceModels <= MAX_STATIC_MESH_LODS);

	if (NumSourceModels == 1)
	{
		// Only one model, 
		GetSourceModel(0).ScreenSize.Default = 1.0f;
	}
	else
	{
		// Use 1080p, 90 degree FOV as a default, as this should not cause runtime regressions in the common case.
		const float HalfFOV = UE_PI * 0.25f;
		const float ScreenWidth = 1920.0f;
		const float ScreenHeight = 1080.0f;
		const FPerspectiveMatrix ProjMatrix(HalfFOV, ScreenWidth, ScreenHeight, 1.0f);
		FBoxSphereBounds Bounds = GetBounds();

		// Multiple models, we should have LOD screen area data.
		for (int32 ModelIndex = 0; ModelIndex < NumSourceModels; ++ModelIndex)
		{
			FStaticMeshSourceModel& SrcModel = GetSourceModel(ModelIndex);

			if (SrcModel.ScreenSize.Default == 0.0f)
			{
				SrcModel.ScreenSize.Default = 1.0f;
				GetRenderData()->ScreenSize[ModelIndex] = SrcModel.ScreenSize.Default;
			}
			else
			{
				// legacy transition screen size was previously a screen AREA fraction using resolution-scaled values, so we need to convert to distance first to correctly calculate the threshold
				const float ScreenArea = SrcModel.ScreenSize.Default * (ScreenWidth * ScreenHeight);
				const float ScreenRadius = FMath::Sqrt(ScreenArea / UE_PI);
				const float ScreenDistance = FMath::Max(ScreenWidth / 2.0f * ProjMatrix.M[0][0], ScreenHeight / 2.0f * ProjMatrix.M[1][1]) * Bounds.SphereRadius / ScreenRadius;

				// Now convert using the query function
				SrcModel.ScreenSize.Default = ComputeBoundsScreenSize(FVector::ZeroVector, Bounds.SphereRadius, FVector(0.0f, 0.0f, ScreenDistance), ProjMatrix);
				GetRenderData()->ScreenSize[ModelIndex] = SrcModel.ScreenSize.Default;
			}
		}
	}
}

void UStaticMesh::GenerateLodsInPackage()
{
	FFormatNamedArguments Args;
	Args.Add(TEXT("StaticMeshName"), FText::FromString(GetName()));
	FStaticMeshStatusMessageContext StatusContext(FText::Format(NSLOCTEXT("Engine", "SavingStaticMeshLODsStatus", "Saving generated LODs for static mesh {StaticMeshName}..."), Args));

	// Get LODGroup info
	ITargetPlatformManagerModule& TargetPlatformManager = GetTargetPlatformManagerRef();
	ITargetPlatform* RunningPlatform = TargetPlatformManager.GetRunningTargetPlatform();
	check(RunningPlatform);
	const FStaticMeshLODSettings& LODSettings = RunningPlatform->GetStaticMeshLODSettings();

	// Generate the reduced models
	IMeshUtilities& MeshUtilities = FModuleManager::Get().LoadModuleChecked<IMeshUtilities>(TEXT("MeshUtilities"));
	if (MeshUtilities.GenerateStaticMeshLODs(this, LODSettings.GetLODGroup(GetLODGroup())))
	{
		// Clear LOD settings
		SetLODGroup(NAME_None, false, false);
		const auto& NewGroup = LODSettings.GetLODGroup(GetLODGroup());
		for (int32 Index = 0; Index < GetNumSourceModels(); ++Index)
		{
			GetSourceModel(Index).ReductionSettings = NewGroup.GetDefaultSettings(0);
		}

		Build(true);

		// Raw mesh is now dirty, so the package has to be resaved
		MarkPackageDirty();
	}
}

#endif // #if WITH_EDITOR

void UStaticMesh::AddSocket(UStaticMeshSocket* Socket)
{
	Sockets.AddUnique(Socket);
}

UStaticMeshSocket* UStaticMesh::FindSocket(FName InSocketName) const
{
	if(InSocketName == NAME_None)
	{
		return NULL;
	}

	for(int32 i=0; i<Sockets.Num(); i++)
	{
		UStaticMeshSocket* Socket = Sockets[i];
		if(Socket && Socket->SocketName == InSocketName)
		{
			return Socket;
		}
	}
	return NULL;
}

void UStaticMesh::RemoveSocket(UStaticMeshSocket* Socket)
{
	Sockets.Remove(Socket);
}

TArray<UStaticMeshSocket*> UStaticMesh::GetSocketsByTag(const FString& InSocketTag) const
{
	TArray<UStaticMeshSocket*> FoundSockets;

	for (int32 i = 0; i < Sockets.Num(); ++i)
	{
		UStaticMeshSocket* Socket = Sockets[i];
		if (Socket && Socket->Tag == InSocketTag)
		{
			FoundSockets.Add(Socket);
		}
	}

	return FoundSockets;
}

ENGINE_API int32 UStaticMesh::GetDefaultMinLOD() const
{
	if (IsMinLodQualityLevelEnable())
	{
		return GetQualityLevelMinLOD().GetDefault();
	}
	else
	{
		return GetMinLOD().GetDefault();
	}
}

ENGINE_API int32 UStaticMesh::GetMinLODIdx(bool bForceLowestLODIdx) const
{
	if (IsMinLodQualityLevelEnable())
	{
		int32 CurrentMinLodQualityLevel = GMinLodQualityLevel;
#if PLATFORM_DESKTOP
		extern ENGINE_API int32 GUseMobileLODBiasOnDesktopES31;
		if (GUseMobileLODBiasOnDesktopES31 != 0 && GMaxRHIFeatureLevel == ERHIFeatureLevel::ES3_1)
		{
			CurrentMinLodQualityLevel = (int32)EPerQualityLevels::Low;
		}
#endif
		return bForceLowestLODIdx ? GetQualityLevelMinLOD().GetLowestValue() : GetQualityLevelMinLOD().GetValue(CurrentMinLodQualityLevel);
	}
	else
	{
		return GetMinLOD().GetValue();
	}
}

ENGINE_API void UStaticMesh::SetMinLODIdx(int32 InMinLOD)
{
	if (IsMinLodQualityLevelEnable())
	{
		SetQualityLevelMinLOD(InMinLOD);
	}
	else
	{
		SetMinLOD(InMinLOD);
	}
}

/** Check the QualitLevel property is enabled for MinLod. */
bool UStaticMesh::IsMinLodQualityLevelEnable() const
{
	return (GEngine && GEngine->UseStaticMeshMinLODPerQualityLevels);
}

void UStaticMesh::OnLodStrippingQualityLevelChanged(IConsoleVariable* Variable){
#if WITH_EDITOR || PLATFORM_DESKTOP
	if (GEngine && GEngine->UseStaticMeshMinLODPerQualityLevels)
	{
		TArray<UStaticMesh*> StaticMeshes;
		for (TObjectIterator<UStaticMesh> It; It; ++It)
		{
			UStaticMesh* StaticMesh = *It;
			if (StaticMesh && StaticMesh->GetQualityLevelMinLOD().PerQuality.Num() > 0)
			{
				StaticMeshes.Add(StaticMesh);
			}
		}

		if (StaticMeshes.Num() > 0)
		{
			FStaticMeshComponentRecreateRenderStateContext Context(StaticMeshes, false);
		}
	}
#endif
}

#if WITH_EDITORONLY_DATA

bool UStaticMesh::IsNaniteEnabled() const
{
	return GetNaniteSettings().bEnabled || IsNaniteForceEnabled();
}

bool UStaticMesh::IsNaniteForceEnabled() const
{
	static const bool bForceEnabled = !!CVarForceEnableNaniteMeshes.GetValueOnAnyThread();
	return bForceEnabled;
}

bool UStaticMesh::IsNaniteAssembly() const
{
	return IsNaniteEnabled() && GetNaniteSettings().NaniteAssemblyData.IsValid();
}

void UStaticMesh::RecacheNaniteAssemblyReferences(bool bWaitForAsyncCompile)
{
	if (AssemblyReferenceCache.Num() > 0)
	{
		ClearCachedNaniteAssemblyReferences();
	}

	if (!IsNaniteAssembly())
	{
		return;
	}

	check(IsInGameThread());

	// Load referenced static meshes for the Nanite assembly build
	// TODO: Nanite-Assemblies - Handle assemblies of assemblies
	// TODO: Nanite-Assemblies - Detect and error on cyclic dependencies

	AssemblyReferenceCache.Reserve(GetNaniteSettings().NaniteAssemblyData.Parts.Num());
	for (const FNaniteAssemblyPart& PartData : GetNaniteSettings().NaniteAssemblyData.Parts)
	{
		FCookLoadScope Scope(ECookLoadType::EditorOnly);
		UStaticMesh* Ref = (UStaticMesh*)StaticLoadAsset(UStaticMesh::StaticClass(), PartData.MeshObjectPath.GetAssetPath());
		if (Ref == nullptr)
		{
			UE_LOGF(
				LogStaticMesh, Warning,
				"Failed to load static mesh %ls when precaching Nanite Assembly references for %ls.",
				*PartData.MeshObjectPath.GetAssetPathString(), *GetPathName()
			);
		}
		AssemblyReferenceCache.Add(Ref);
	}

	for (UStaticMesh* Ref : AssemblyReferenceCache)
	{
		if (Ref)
		{
			if (bWaitForAsyncCompile)
			{
				Ref->WaitUntilAsyncPropertyReleased(EStaticMeshAsyncProperties::All);
			}
		#if WITH_EDITOR
			Ref->OnPostMeshBuild().AddUObject(this, &UStaticMesh::OnNaniteAssemblyReferenceBuilt);
		#endif
		}
	}
}

bool UStaticMesh::HasCachedNaniteAssemblyReferences() const
{
	return !IsNaniteAssembly() || AssemblyReferenceCache.Num() == GetNaniteSettings().NaniteAssemblyData.Parts.Num();
}

const TArray<TObjectPtr<UStaticMesh>>& UStaticMesh::GetCachedNaniteAssemblyReferences() const
{
	return AssemblyReferenceCache;
}

void UStaticMesh::ClearCachedNaniteAssemblyReferences()
{
	check(IsInGameThread());

#if WITH_EDITOR
	for (UStaticMesh* Ref : AssemblyReferenceCache)
	{
		if (Ref)
		{
			Ref->OnPostMeshBuild().RemoveAll(this);
		}
	}
#endif
	AssemblyReferenceCache.Empty();
}

#endif

#if WITH_EDITOR

void UStaticMesh::OnNaniteAssemblyReferenceBuilt(UStaticMesh* Reference)
{
	if (!IsCompiling())
	{
		// Check to see if we need to rebuild and recache render data for the running platform
		const ITargetPlatform* TargetPlatform = GetTargetPlatformManagerRef().GetRunningTargetPlatform();
		const FStaticMeshLODGroup& PlatformLODGroup = TargetPlatform->GetStaticMeshLODSettings().GetLODGroup(GetLODGroup());
		FString CurrentDDCKey = UE::Private::StaticMesh::BuildStaticMeshDerivedDataKey(TargetPlatform, this, PlatformLODGroup);
		FStaticMeshRenderData* PlatformRenderData = GetRenderData();
		while (PlatformRenderData && PlatformRenderData->DerivedDataKey != CurrentDDCKey)
		{
			PlatformRenderData = PlatformRenderData->NextCachedRenderData.Get();
		}

		if (PlatformRenderData == nullptr)
		{
			Build();
		}
	}
}

#endif

void UStaticMesh::UpdateStaticMaterialsInfoCache(const TArray<FStaticMaterial>& InMaterials)
{
	FScopeLock Lock(&MaterialsInfoCriticalSection);
	StaticMaterialsInfoCache.Reset();
	for (int32 Index = 0; Index < InMaterials.Num(); Index++)
	{
		FName MaterialSlotName = InMaterials[Index].MaterialSlotName;
		FString MaterialObjectPath = InMaterials[Index].MaterialInterface ? InMaterials[Index].MaterialInterface->GetPathName() : FString();
		StaticMaterialsInfoCache.Add(FStaticMaterialMinimalInfo(MaterialSlotName, MaterialObjectPath));
	}
}

#define MATERIAL_SLOT_DELIMITER TEXT(">")
#define MATERIAL_SLOT_PAIR_DELIMITER TEXT(",")
const FString UStaticMesh::ExportStaticMaterialsInfo() const
{
	FScopeLock Lock(&MaterialsInfoCriticalSection);
	FString OutText;
	for (const FStaticMaterialMinimalInfo& Slot : StaticMaterialsInfoCache)
	{
		OutText += Slot.MaterialSlotName.ToString();
		OutText += MATERIAL_SLOT_PAIR_DELIMITER;
		OutText += FString::Printf(TEXT("\'%s\'"), *Slot.MaterialObjectPath);  // Add quotes so object path is recognized during asset header patching
		OutText += MATERIAL_SLOT_DELIMITER;
	}
	return OutText;
}

const TArray<FStaticMaterialMinimalInfo> UStaticMesh::ImportStaticMaterialsInfo(const FString& InData)
{
	TArray<FStaticMaterialMinimalInfo> OutArray;
	if (!InData.IsEmpty())
	{
		TArray<FString> Slot;
		const int32 NumPairs = InData.ParseIntoArray(Slot, MATERIAL_SLOT_DELIMITER, true);
		for (int32 Index = 0; Index < NumPairs; Index++)
		{
			TArray<FString> Pair;
			int32 NumEntries = Slot[Index].ParseIntoArray(Pair, MATERIAL_SLOT_PAIR_DELIMITER, true);
			if (NumEntries == 2)
			{
				// Remove quotes
				Pair[1].RemoveFromStart(TEXT("\'"));
				Pair[1].RemoveFromEnd(TEXT("\'"));

				OutArray.Add(FStaticMaterialMinimalInfo(FName(Pair[0]), Pair[1]));
			}
		}
	}
	return OutArray;
}

bool UStaticMesh::CanMeshPaintTextureColors() const
{
	if (StaticMeshPaintSupport == EStaticMeshPaintSupport::Default)
	{
		return CVarStaticMeshDefaultMeshPaintTextureSupport.GetValueOnGameThread();
	}
	return StaticMeshPaintSupport != EStaticMeshPaintSupport::Disabled;
}

/*-----------------------------------------------------------------------------
UStaticMeshSocket
-----------------------------------------------------------------------------*/

UStaticMeshSocket::UStaticMeshSocket(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	RelativeScale = FVector(1.0f, 1.0f, 1.0f);
#if WITH_EDITORONLY_DATA
	bSocketCreatedAtImport = false;
#endif
}

/** Utility that returns the current matrix for this socket. */
bool UStaticMeshSocket::GetSocketMatrix(FMatrix& OutMatrix, UStaticMeshComponent const* MeshComp) const
{
	check( MeshComp );
	OutMatrix = FScaleRotationTranslationMatrix( RelativeScale, RelativeRotation, RelativeLocation ) * MeshComp->GetComponentTransform().ToMatrixWithScale();
	return true;
}

bool UStaticMeshSocket::GetSocketTransform(FTransform& OutTransform, class UStaticMeshComponent const* MeshComp) const
{
	check( MeshComp );
	OutTransform = FTransform(RelativeRotation, RelativeLocation, RelativeScale) * MeshComp->GetComponentTransform();
	return true;
}

bool UStaticMeshSocket::AttachActor(AActor* Actor,  UStaticMeshComponent* MeshComp) const
{
	bool bAttached = false;

	// Don't support attaching to own socket
	if (Actor != MeshComp->GetOwner() && Actor->GetRootComponent())
	{
		FMatrix SocketTM;
		if( GetSocketMatrix( SocketTM, MeshComp ) )
		{
			Actor->Modify();

			Actor->SetActorLocation(SocketTM.GetOrigin(), false);
			Actor->SetActorRotation(SocketTM.Rotator());
			Actor->GetRootComponent()->AttachToComponent(MeshComp, FAttachmentTransformRules::SnapToTargetNotIncludingScale, SocketName);

#if WITH_EDITOR
			if (GIsEditor)
			{
				Actor->PreEditChange(NULL);
				Actor->PostEditChange();
			}
#endif // WITH_EDITOR

			bAttached = true;
		}
	}
	return bAttached;
}

#if WITH_EDITOR
void UStaticMeshSocket::PostEditChangeProperty( FPropertyChangedEvent& PropertyChangedEvent )
{
	Super::PostEditChangeProperty( PropertyChangedEvent );

	if( PropertyChangedEvent.Property )
	{
		ChangedEvent.Broadcast( this, PropertyChangedEvent.MemberProperty );
	}
}
#endif

void UStaticMeshSocket::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

	Ar.UsingCustomVersion(FFrameworkObjectVersion::GUID);

	if(Ar.CustomVer(FFrameworkObjectVersion::GUID) < FFrameworkObjectVersion::MeshSocketScaleUtilization)
	{
		// Set the relative scale to 1.0. As it was not used before this should allow existing data
		// to work as expected.
		RelativeScale = FVector(1.0f, 1.0f, 1.0f);
	}
}

FStaticMeshCompilationContext::FStaticMeshCompilationContext() 
{
	// Remember if the editor was loading a package when initiating the build so that we can temporarily restore that state when 
	//  executing FinishBuildInternal on the game thread at the end of the build :
	bIsEditorLoadingPackage = UE::GetIsEditorLoadingPackage();
}

#undef LOCTEXT_NAMESPACE
