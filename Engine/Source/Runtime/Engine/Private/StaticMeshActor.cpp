// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	StaticMeshActorBase.cpp: Static mesh actor base class implementation.
=============================================================================*/

#include "Engine/StaticMeshActor.h"
#include "UObject/FrameworkObjectVersion.h"
#include "Engine/CollisionProfile.h"
#include "Engine/OverlapResult.h"
#include "Engine/World.h"
#include "Logging/MessageLog.h"
#include "Misc/UObjectToken.h"
#include "Misc/MapErrors.h"
#include "PhysicsEngine/BodySetup.h"
#include "StaticMeshComponentLODInfo.h"
#include "StaticMeshResources.h"
#include "Engine/StaticMesh.h"
#include "UObject/UnrealType.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(StaticMeshActor)


#define LOCTEXT_NAMESPACE "StaticMeshActor"
FName AStaticMeshActor::StaticMeshComponentName(TEXT("StaticMeshComponent0"));

AStaticMeshActor::AStaticMeshActor(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	SetCanBeDamaged(false);

	StaticMeshComponent = CreateDefaultSubobject<UStaticMeshComponent>(StaticMeshComponentName);
	StaticMeshComponent->SetCollisionProfileName(UCollisionProfile::BlockAll_ProfileName);
	StaticMeshComponent->Mobility = EComponentMobility::Static;
	StaticMeshComponent->SetGenerateOverlapEvents(false);
	StaticMeshComponent->bUseDefaultCollision = true;

	RootComponent = StaticMeshComponent;
	
	// Only actors that are literally static mesh actors can be placed in clusters, native subclasses or BP subclasses are not safe by default
	bCanBeInCluster = (GetClass() == AStaticMeshActor::StaticClass());
}

void AStaticMeshActor::PostInitializeComponents()
{
	Super::PostInitializeComponents();

	// Since we allow AStaticMeshActor to specify whether it replicates via bStaticMeshReplicateMovement - per placed instance
	// and we do the normal SetReplicates call in PostInitProperties, before instanced properties are serialized in, 
	// we need to do this here. 
	//
	// This is a short term fix until we find a better play for SetReplicates to be called in AActor.
	if (GetLocalRole() == ROLE_Authority && bStaticMeshReplicateMovement)
	{
		bReplicates = false;
		SetRemoteRoleForBackwardsCompat(ROLE_SimulatedProxy);
		SetReplicates(true);
	}

	SetPhysicsReplicationMode(StaticMeshPhysicsReplicationMode);
}

FString AStaticMeshActor::GetDetailedInfoInternal() const
{
	return StaticMeshComponent ? StaticMeshComponent->GetDetailedInfoInternal() : TEXT("No_StaticMeshComponent");
}

void AStaticMeshActor::SetMobility(EComponentMobility::Type InMobility)
{
	if(StaticMeshComponent)
	{
		StaticMeshComponent->SetMobility(InMobility);
	}
}

void AStaticMeshActor::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);
#if WITH_EDITOR
	Ar.UsingCustomVersion(FFrameworkObjectVersion::GUID);
#endif
}

#if WITH_EDITOR

void AStaticMeshActor::LoadedFromAnotherClass(const FName& OldClassName)
{
	Super::LoadedFromAnotherClass(OldClassName);

	if(GetLinkerUEVersion() < VER_UE4_REMOVE_STATICMESH_MOBILITY_CLASSES)
	{
		static FName InterpActor_NAME("InterpActor");
		static FName PhysicsActor_NAME("PhysicsActor");

		if(OldClassName == InterpActor_NAME)
		{
			StaticMeshComponent->Mobility = EComponentMobility::Movable;
			StaticMeshComponent->SetCollisionProfileName(UCollisionProfile::BlockAllDynamic_ProfileName);
		}
		else if(OldClassName == PhysicsActor_NAME)
		{
			StaticMeshComponent->Mobility = EComponentMobility::Movable;
			StaticMeshComponent->SetCollisionProfileName(UCollisionProfile::PhysicsActor_ProfileName);
			StaticMeshComponent->BodyInstance.bSimulatePhysics = true;

			SetCanBeDamaged(true);
			SetReplicatingMovement(true);
			SetRemoteRoleForBackwardsCompat(ROLE_SimulatedProxy);
			bReplicates = true;
		}
	}
}


void AStaticMeshActor::PostEditChangeChainProperty(FPropertyChangedChainEvent& PropertyChangedEvent)
{
	Super::PostEditChangeChainProperty(PropertyChangedEvent);

	if (PropertyChangedEvent.Property != nullptr)
	{
		// If we want to replicate movement, set RemoteRole to match
		if (PropertyChangedEvent.Property->GetFName() == FName("bStaticMeshReplicateMovement"))
		{
			SetReplicatingMovement(bStaticMeshReplicateMovement);
			SetReplicates(IsReplicatingMovement());
		}
	}

	// Get 'deepest' property name we changed.
	const FName TailPropName = PropertyChangedEvent.PropertyChain.GetTail()->GetValue()->GetFName();
	static FName SimulatePhysics_NAME("bSimulatePhysics");
	if(TailPropName == SimulatePhysics_NAME)
	{
		SetCanBeDamaged(StaticMeshComponent->BodyInstance.bSimulatePhysics);
	}
}


bool AStaticMeshActor::GetReferencedContentObjects( TArray<UObject*>& Objects ) const
{
	Super::GetReferencedContentObjects(Objects);

	if (StaticMeshComponent && StaticMeshComponent->GetStaticMesh())
	{
		Objects.Add(StaticMeshComponent->GetStaticMesh());
	}
	return true;
}

void AStaticMeshActor::PostLoad()
{
	Super::PostLoad();

	if (StaticMeshComponent && GetLinkerCustomVersion(FFrameworkObjectVersion::GUID) < FFrameworkObjectVersion::UseBodySetupCollisionProfile)
	{
		//For all existing content we check whether we need to mark collision profile for override.
		if (UBodySetup* BodySetup = StaticMeshComponent->GetBodySetup())
		{
			if (BodySetup->DefaultInstance.GetCollisionProfileName() != StaticMeshComponent->GetCollisionProfileName())
			{
				StaticMeshComponent->bUseDefaultCollision = false;
			}
		}
	}
	
}

void AStaticMeshActor::CheckForErrors()
{
	Super::CheckForErrors();

	FMessageLog MapCheck("MapCheck");

	if( !StaticMeshComponent )
	{
		FFormatNamedArguments Arguments;
		Arguments.Add(TEXT("ActorName"), FText::FromString(GetName()));
		MapCheck.Warning()
			->AddToken(FUObjectToken::Create(this))
			->AddToken(FTextToken::Create(FText::Format(LOCTEXT( "MapCheck_Message_StaticMeshComponent", "Static mesh actor {ActorName} has NULL StaticMeshComponent property - please delete" ), Arguments)))
			->AddToken(FMapErrorToken::Create(FMapErrors::StaticMeshComponent));
	}
	else if( StaticMeshComponent->GetStaticMesh() != nullptr )
	{
		FCollisionQueryParams SphereParams(SCENE_QUERY_STAT(CheckForErrors), false, this);

		TArray<FOverlapResult> Overlaps;
		GetWorld()->OverlapMultiByChannel(Overlaps, GetActorLocation(), FQuat::Identity, ECC_Pawn, FCollisionShape::MakeSphere(1.f), SphereParams);

		for ( const FOverlapResult& Overlap : Overlaps )
		{
			if (Overlap.OverlapObjectHandle.IsValid() && Overlap.OverlapObjectHandle != this && Overlap.OverlapObjectHandle.DoesRepresentClass(AStaticMeshActor::StaticClass()) &&
				(Overlap.OverlapObjectHandle.GetLocation() - GetActorLocation()).IsNearlyZero() &&
				(Overlap.OverlapObjectHandle.GetRotation() == GetActorRotation()))
			{
				AStaticMeshActor* A = Overlap.OverlapObjectHandle.FetchActor<AStaticMeshActor>();
				check(A);

				if (A->StaticMeshComponent && (A->StaticMeshComponent->GetStaticMesh() == StaticMeshComponent->GetStaticMesh()) && (A->StaticMeshComponent->GetRelativeScale3D() == StaticMeshComponent->GetRelativeScale3D()))
				{
					FFormatNamedArguments Arguments;
						Arguments.Add(TEXT("ActorName0"), FText::FromString(GetName()));
						Arguments.Add(TEXT("ActorName1"), FText::FromString(A->GetName()));
						MapCheck.Warning()
						->AddToken(FUObjectToken::Create(this))
						->AddToken(FTextToken::Create(FText::Format(LOCTEXT("MapCheck_Message_SameLocation", "{ActorName0} is in the same location as {ActorName1}"), Arguments)))
						->AddToken(FMapErrorToken::Create(FMapErrors::SameLocation));
				}
			}
		}

		// If this mesh has overridden vertex colors, make sure that it matches up with the original mesh
		{
			int32 NumLODs = StaticMeshComponent->GetStaticMesh()->GetNumLODs();
			for( int32 CurLODIndex = 0; CurLODIndex < NumLODs; ++CurLODIndex )
			{
				// Make sure the color vertex buffer's vertex count matches the rest of the mesh!
				const FStaticMeshLODResources& LODRenderData = StaticMeshComponent->GetStaticMesh()->GetRenderData()->LODResources[ CurLODIndex ];

				// We may not have cached any LODData for this particular LODModel yet, so make sure that
				// we're in bounds before accessing the current model's LOD info
				if( StaticMeshComponent->LODData.Num() > CurLODIndex )
				{
					const FStaticMeshComponentLODInfo& ComponentLODInfo = StaticMeshComponent->LODData[ CurLODIndex ];

					if( ComponentLODInfo.OverrideVertexColors &&
						ComponentLODInfo.OverrideVertexColors->GetNumVertices() != LODRenderData.GetNumVertices() )
					{
						FFormatNamedArguments Arguments;
						Arguments.Add(TEXT("ActorName"), FText::FromString(GetName()));
						Arguments.Add(TEXT("LODIndex"), CurLODIndex);
						Arguments.Add(TEXT("StaticMeshName"), FText::FromString(StaticMeshComponent->GetStaticMesh()->GetName()));

						// Uh oh, looks like the original mesh was changed since this instance's vertex
						// colors were painted down
						MapCheck.Warning()
							->AddToken(FUObjectToken::Create(this))
							->AddToken(FTextToken::Create(FText::Format( LOCTEXT("MapCheck_Message_VertexColorsNotMatchOriginalMesh", "{ActorName} (LOD {LODIndex}) has hand-painted vertex colors that no longer match the original StaticMesh ({StaticMeshName})" ), Arguments ) ))
							->AddToken(FMapErrorToken::Create(FMapErrors::VertexColorsNotMatchOriginalMesh));
					}
				}
			}
		}
	}
}

#endif // WITH_EDITOR

#undef LOCTEXT_NAMESPACE


