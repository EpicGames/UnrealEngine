// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	StaticMeshBuild.cpp: Static mesh building.
=============================================================================*/

#include "Engine/StaticMeshSourceData.h"
#include "Math/GenericOctreePublic.h"
#include "EngineLogs.h"
#include "Math/GenericOctree.h"
#include "Engine/StaticMesh.h"
#include "MeshDescription.h"
#include "StaticMeshResources.h"
#include "PhysicsEngine/BodySetup.h"
#include "DistanceFieldAtlas.h"
#include "MeshCardBuild.h"
#include "AI/Navigation/NavCollisionBase.h"
#include "SceneInterface.h"

#if WITH_EDITOR
#include "Async/Async.h"
#include "ObjectCacheContext.h"
#include "IMeshBuilderModule.h"
#include "IMeshReductionManagerModule.h"
#include "RenderingThread.h"
#include "StaticMeshCompiler.h"
#include "MeshUtilitiesCommon.h"
#include "Misc/ScopedSlowTask.h"

#include "Rendering/StaticLightingSystemInterface.h"
#endif // #if WITH_EDITOR

#define LOCTEXT_NAMESPACE "StaticMeshEditor"

#if WITH_EDITOR
/**
 * Check the render data for the provided mesh and return true if the mesh
 * contains degenerate tangent bases.
 */
static bool HasBadNTB(UStaticMesh* Mesh, bool &bZeroNormals, bool &bZeroTangents, bool &bZeroBinormals)
{
	bZeroTangents = false;
	bZeroNormals = false;
	bZeroBinormals = false;
	bool bBadTangents = false;
	if (Mesh && Mesh->GetRenderData())
	{
		int32 NumLODs = Mesh->GetNumLODs();
		for (int32 LODIndex = 0; LODIndex < NumLODs; ++LODIndex)
		{
			FStaticMeshLODResources& LOD = Mesh->GetRenderData()->LODResources[LODIndex];
			int32 NumVerts = LOD.VertexBuffers.PositionVertexBuffer.GetNumVertices();
			for (int32 VertIndex = 0; VertIndex < NumVerts; ++VertIndex)
			{
				const FVector3f TangentX = LOD.VertexBuffers.StaticMeshVertexBuffer.VertexTangentX(VertIndex);
				const FVector3f TangentY = LOD.VertexBuffers.StaticMeshVertexBuffer.VertexTangentY(VertIndex);
				const FVector3f TangentZ = LOD.VertexBuffers.StaticMeshVertexBuffer.VertexTangentZ(VertIndex);
				
				if (TangentX.IsNearlyZero(UE_KINDA_SMALL_NUMBER))
				{
					bZeroTangents = true;
				}
				if (TangentY.IsNearlyZero(UE_KINDA_SMALL_NUMBER))
				{
					bZeroBinormals = true;
				}
				if (TangentZ.IsNearlyZero(UE_KINDA_SMALL_NUMBER))
				{
					bZeroNormals = true;
				}
				if ((TangentX - TangentZ).IsNearlyZero(1.0f / 255.0f))
				{
					bBadTangents = true;
				}
			}
		}
	}
	return bBadTangents;
}

bool UStaticMesh::CanBuild() const
{
	if (IsTemplate())
	{
		return false;
	}

	if (GetNumSourceModels() <= 0)
	{
		UE_LOGF(LogStaticMesh, Warning, "Static mesh has no source models: %ls", *GetPathName());
		return false;
	}

	if (GetNumSourceModels() > MAX_STATIC_MESH_LODS)
	{
		UE_LOGF(LogStaticMesh, Warning, "Cannot build LOD %d.  The maximum allowed is %d.  Skipping.", GetNumSourceModels(), MAX_STATIC_MESH_LODS);
		return false;
	}

	// Mesh descriptions are unlikely at runtime but UStaticMesh::Build can still be called
	if (FApp::IsGame())
	{
		if (!IsMeshDescriptionValid(0))
		{
			return false;
		}
	}

	return true;
}

static TAutoConsoleVariable<int32> CVarStaticMeshDisableThreadedBuild(
	TEXT("r.StaticMesh.DisableThreadedBuild"),
	0,
	TEXT("Activate to force static mesh building from a single thread.\n"),
	ECVF_Default);

#endif // #if WITH_EDITOR

void UStaticMesh::Build(const FBuildParameters& BuildParameters)
{
#if WITH_EDITOR
	FFormatNamedArguments Args;
	Args.Add(TEXT("Path"), FText::FromString(GetPathName()));
	const FText StatusUpdate = FText::Format(LOCTEXT("BeginStaticMeshBuildingTask", "({Path}) Building"), Args);
	FScopedSlowTask StaticMeshBuildingSlowTask(1, StatusUpdate);
	if (!BuildParameters.bInSilent)
	{
		StaticMeshBuildingSlowTask.MakeDialogDelayed(1.0f);
	}
	StaticMeshBuildingSlowTask.EnterProgressFrame(1);
#endif // #if WITH_EDITOR

	BatchBuild({ this }, BuildParameters, nullptr);
}

void UStaticMesh::BatchBuild(const TArray<UStaticMesh*>& InStaticMeshes, const FBuildParameters& BuildParameters, TFunction<bool(UStaticMesh*)> InProgressCallback)
{
#if WITH_EDITOR
	check(IsInGameThread());
	TRACE_CPUPROFILER_EVENT_SCOPE(UStaticMesh::BatchBuild);

	FObjectCacheContextScope ObjectCacheScope;

	TSet<UStaticMesh*> StaticMeshesToProcess;
	StaticMeshesToProcess.Reserve(InStaticMeshes.Num());

	bool bIsFinishCompilationRequired = false;
	for (UStaticMesh* StaticMesh : InStaticMeshes)
	{
		if (StaticMesh && StaticMesh->CanBuild())
		{
			bIsFinishCompilationRequired |= StaticMesh->IsCompiling();
			StaticMeshesToProcess.Add(StaticMesh);
		}
	}

	if (StaticMeshesToProcess.Num())
	{
		if (bIsFinishCompilationRequired)
		{
			FStaticMeshCompilingManager::Get().FinishCompilation(StaticMeshesToProcess.Array());
		}

		// Make sure the target platform is properly initialized before accessing it from multiple threads
		ITargetPlatformManagerModule& TargetPlatformManager = GetTargetPlatformManagerRef();
		ITargetPlatform* RunningPlatform = TargetPlatformManager.GetRunningTargetPlatform();
		check(RunningPlatform);

		// Ensure those modules are loaded on the main thread - we'll need them in async tasks
		FModuleManager::Get().LoadModuleChecked<IMeshUtilities>(TEXT("MeshUtilities"));
		FModuleManager::Get().LoadModuleChecked<IMeshReductionManagerModule>(TEXT("MeshReductionInterface"));
		IMeshBuilderModule::GetForRunningPlatform();
		for (const ITargetPlatform* TargetPlatform : TargetPlatformManager.GetActiveTargetPlatforms())
		{
			IMeshBuilderModule::GetForPlatform(TargetPlatform);
		}

		if (GDistanceFieldAsyncQueue)
		{
			GDistanceFieldAsyncQueue->CancelBuilds(StaticMeshesToProcess);
		}

		if (GCardRepresentationAsyncQueue)
		{
			GCardRepresentationAsyncQueue->CancelBuilds(StaticMeshesToProcess);
		}

		TMap<UStaticMesh*, TArray<IStaticMeshComponent*>> StaticMeshComponents;
		StaticMeshComponents.Reserve(InStaticMeshes.Num());

		TSet<FSceneInterface*> Scenes;

		for (UStaticMesh* StaticMesh : InStaticMeshes)
		{
			if (StaticMesh)
			{
				StaticMeshComponents.Add(StaticMesh);

				for (IStaticMeshComponent* Component : ObjectCacheScope.GetContext().GetStaticMeshComponents(StaticMesh))
				{
					Component->PreStaticMeshCompilation();

					IPrimitiveComponent* PrimitiveComponent = Component->GetPrimitiveComponentInterface();

					// Detach all instances of those static meshes from the scene.
					if (PrimitiveComponent->IsRenderStateCreated())
					{
						PrimitiveComponent->DestroyRenderState();
						Scenes.Add(PrimitiveComponent->GetScene());
					}
					
					if (PrimitiveComponent->IsRegistered())
					{
						StaticMeshComponents[StaticMesh].Add(Component);
					}
				}
			}
		}

		// Only flush rendering commands if necessary
		if (Scenes.Num())
		{
			UpdateAllPrimitiveSceneInfosForScenes(MoveTemp(Scenes));

			// Flush the rendering commands generated by the detachments.
			// The static mesh scene proxies reference the UStaticMesh, and this ensures that they are cleaned up before the UStaticMesh changes.
			FlushRenderingCommands();
		}

		auto FinalizeStaticMesh = 
			[&Scenes, &StaticMeshComponents](UStaticMesh* StaticMesh)
			{
				if (TArray<IStaticMeshComponent*>* MeshComponents = StaticMeshComponents.Find(StaticMesh))
				{
					for (IStaticMeshComponent* Component : *MeshComponents)
					{
						IPrimitiveComponent* PrimitiveComponent = Component->GetPrimitiveComponentInterface();

						if (PrimitiveComponent->IsRegistered() && !PrimitiveComponent->IsRenderStateCreated() && PrimitiveComponent->ShouldCreateRenderState())
						{
							PrimitiveComponent->CreateRenderState(nullptr);
							Scenes.Add(PrimitiveComponent->GetScene());
						}
					}
				}
			};

		auto LaunchAsyncBuild =
			[BuildParameters](UStaticMesh* StaticMesh)
			{
				// Only launch async compile if errors are not required
				if (BuildParameters.OutErrors == nullptr && FStaticMeshCompilingManager::Get().IsAsyncCompilationAllowed(StaticMesh))
				{
					const int64 BuildRequiredMemory = StaticMesh->GetBuildRequiredMemoryEstimate();
					TUniquePtr<FStaticMeshBuildContext> Context = MakeUnique<FStaticMeshBuildContext>(BuildParameters);
					StaticMesh->BeginBuildInternal(Context.Get());

					FQueuedThreadPool* StaticMeshThreadPool = FStaticMeshCompilingManager::Get().GetThreadPool();
					EQueuedWorkPriority BasePriority = FStaticMeshCompilingManager::Get().GetBasePriority(StaticMesh);
					check(StaticMesh->AsyncTask == nullptr);
					StaticMesh->AsyncTask = MakeUnique<FStaticMeshAsyncBuildTask>(StaticMesh, MoveTemp(Context));
					if (!StaticMesh->HasAnyDependenciesCompiling())
					{
						StaticMesh->AsyncTask->StartBackgroundTask(StaticMeshThreadPool, BasePriority, EQueuedWorkFlags::DoNotRunInsideBusyWait, BuildRequiredMemory, TEXT("StaticMesh"));
					}

					if (StaticMesh->IsNaniteAssembly())
					{
						// This static mesh has other static mesh dependencies that the compiler must keep track of
						FStaticMeshCompilingManager::Get().AddStaticMeshesWithDependencies({ StaticMesh });
					}
					else
					{
						FStaticMeshCompilingManager::Get().AddStaticMeshes({ StaticMesh });
					}
					return true;
				}

				return false;
			};

		if (StaticMeshesToProcess.Num() > 1 && CVarStaticMeshDisableThreadedBuild.GetValueOnAnyThread() == 0)
		{
			FCriticalSection OutErrorsLock;

			struct FStaticMeshTask
			{
				UStaticMesh*  StaticMesh;
				TFuture<bool> Future;

				FStaticMeshTask(UStaticMesh* InStaticMesh, TFuture<bool>&& InFuture)
					: StaticMesh(InStaticMesh)
					, Future(MoveTemp(InFuture))
				{
				}
			};

			// Start async tasks to build the static meshes in parallel
			TArray<FStaticMeshTask> AsyncTasks;
			AsyncTasks.Reserve(StaticMeshesToProcess.Num());
			std::atomic<bool> bCancelled(false);

			for (UStaticMesh* StaticMesh : StaticMeshesToProcess)
			{
				// Preferably launch as an async build, if it fails, fallback on the old behavior
				const bool bAsyncBuildSucceeded = LaunchAsyncBuild(StaticMesh);
				if (!bAsyncBuildSucceeded)
				{
					StaticMesh->BeginBuildInternal();

					AsyncTasks.Emplace(
						StaticMesh,
						Async(
							EAsyncExecution::LargeThreadPool,
							[StaticMesh, BuildParameters, &OutErrorsLock, &bCancelled]()
							{
								if (bCancelled.load(std::memory_order_relaxed))
								{
									return false;
								}

								TArray<FText> Errors;
								const bool bHasRenderDataChanged = StaticMesh->ExecuteBuildInternal(BuildParameters);
								if (BuildParameters.OutErrors)
								{
									FScopeLock ScopeLock(&OutErrorsLock);
									BuildParameters.OutErrors->Append(Errors);
								}

								return bHasRenderDataChanged;
							}
						)
					);
				}
			}

			for (FStaticMeshTask& Task : AsyncTasks)
			{
				if (InProgressCallback && !InProgressCallback(Task.StaticMesh))
				{
					bCancelled = true;
				}

				// Wait the result of the async task
				const bool bHasRenderDataChanged = Task.Future.Get();

				Task.StaticMesh->FinishBuildInternal(StaticMeshComponents.FindChecked(Task.StaticMesh), bHasRenderDataChanged);
				FinalizeStaticMesh(Task.StaticMesh);
			}
		}
		else
		{
			for (UStaticMesh* StaticMesh : StaticMeshesToProcess)
			{
				// Preferably launch as an async build, if it fails, fallback on the old behavior
				const bool bAsyncBuildSucceeded = LaunchAsyncBuild(StaticMesh);
				if (!bAsyncBuildSucceeded)
				{
					if (InProgressCallback && !InProgressCallback(StaticMesh))
					{
						break;
					}

					StaticMesh->BeginBuildInternal();

					const bool bHasRenderDataChanged = StaticMesh->ExecuteBuildInternal(BuildParameters);

					StaticMesh->FinishBuildInternal(StaticMeshComponents.FindChecked(StaticMesh), bHasRenderDataChanged);
					FinalizeStaticMesh(StaticMesh);
				}
			}
		}

		if (Scenes.Num())
		{
			UpdateAllPrimitiveSceneInfosForScenes(MoveTemp(Scenes));
		}
	}
#else
	UE_LOGF(LogStaticMesh, Fatal, "UStaticMesh::Build should not be called on non-editor builds.");
#endif
}

void UStaticMesh::Build(bool bInSilent, TArray<FText>* OutErrors)
{
	FBuildParameters BuildParameters;
	BuildParameters.bInSilent = bInSilent;
	BuildParameters.OutErrors = OutErrors;
	Build(BuildParameters);
}

void UStaticMesh::BatchBuild(const TArray<UStaticMesh*>& InStaticMeshes, bool bInSilent, TFunction<bool(UStaticMesh*)> InProgressCallback, TArray<FText>* OutErrors)
{
	FBuildParameters BuildParameters;
	BuildParameters.bInSilent = bInSilent;
	BuildParameters.OutErrors = OutErrors;
	BatchBuild(InStaticMeshes, BuildParameters, InProgressCallback);
}

#if WITH_EDITOR

extern const FString& GetStaticMeshDerivedDataVersion();
void UStaticMesh::BeginBuildInternal(FStaticMeshBuildContext* Context)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UStaticMesh::PreBuildInternal);
	check(AsyncTask == nullptr);

	// Make sure every static FString's are built and cached on the main thread
	// before trying to access it from multiple threads
	GetStaticMeshDerivedDataVersion();

	PreMeshBuild.Broadcast(this);

	if (!HasCachedNaniteAssemblyReferences())
	{
		RecacheNaniteAssemblyReferences(false);
	}
	
	// Ensure we have a bodysetup.
	CreateBodySetup();
	check(GetBodySetup() != nullptr);
	
	// Make sure the object is created on the game-thread before going async
	if (bHasNavigationData && GetBodySetup() != nullptr)
	{
		if (GetNavCollision() == nullptr)
		{
			SetNavCollision(UNavCollisionBase::ConstructNew(*this));
		}
	}

	// Release the static mesh's resources.
	ReleaseResources();

	// Flush the resource release commands to the rendering thread to ensure that the build doesn't occur while a resource is still
	// allocated, and potentially accessing the UStaticMesh.
	ReleaseResourcesFence.Wait();

	// Having a context means we're running async
	if (Context)
	{
		// Lock all the async properties that could be modified during async build
		AcquireAsyncProperty();

		// Ensure that anything we perform here doesn't cause stalls
		FStaticMeshAsyncBuildScope AsyncBuildScope(this);

		// Extended bounds are super important to avoid many stalls since a lot of different
		// systems rely on mesh bounds. If mesh description is available and doesn't require being
		// loaded, compute bounds right now so we can unlock the property before going async.
		if (IsNaniteAssembly())
		{
			CachedMeshDescriptionBounds.Reset(); // Nanite Assembly bounds must come from Nanite build
		}
		else if (IsSourceModelValid(0) && GetSourceModel(0).GetCachedMeshDescription() != nullptr)
		{
			CachedMeshDescriptionBounds = GetSourceModel(0).GetCachedMeshDescription()->GetBounds();
		}

		// CommitMeshDescription will also fill out CachedMeshDescriptionBounds
		// so the bounds might be available even if ClearMeshDescriptions has been called.
		if (CachedMeshDescriptionBounds.IsSet())
		{
			CalculateExtendedBounds();
			Context->bShouldComputeExtendedBounds = false;

			// Release the property now that it contains valid values to avoid game-thread stalls while we do the rest async
			ReleaseAsyncProperty(EStaticMeshAsyncProperties::ExtendedBounds);
		}
		else
		{
			Context->bShouldComputeExtendedBounds = true;
		}
	}
}

bool UStaticMesh::ExecuteBuildInternal(const FBuildParameters& BuildParameters)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UStaticMesh::ExecuteBuildInternal);

#if WITH_EDITORONLY_DATA
	check(EditableMesh_DEPRECATED == nullptr);
#endif

	FFormatNamedArguments Args;
	Args.Add( TEXT("Path"), FText::FromString( GetPathName() ) );
	const FText StatusUpdate = FText::Format( LOCTEXT("BeginStaticMeshBuildingTask", "({Path}) Building"), Args );
	FScopedSlowTask StaticMeshBuildingSlowTask(1, StatusUpdate);
	StaticMeshBuildingSlowTask.EnterProgressFrame(1);

	// Remember the derived data key of our current render data if any.
	FString ExistingDerivedDataKey = GetRenderData() ? GetRenderData()->DerivedDataKey : TEXT("");

	// Regenerating UVs for lightmaps, use the latest version
	SetLightmapUVVersion((int32)ELightmapUVVersion::Latest);

	// Free existing render data and recache.
	CacheDerivedData();

	if (UE::Tasks::FCancellationTokenScope::IsCurrentWorkCanceled())
	{
		return false;
	}

	PrepareDerivedDataForActiveTargetPlatforms();

	if (UE::Tasks::FCancellationTokenScope::IsCurrentWorkCanceled())
	{
		return false;
	}

	if (BuildParameters.bInEnforceLightmapRestrictions)
	{
		EnforceLightmapRestrictions();
	}

	if (BuildParameters.bInRebuildUVChannelData)
	{
		UpdateUVChannelData(true);
	}

	// InitResources will send commands to other threads that will
	// use our RenderData, we must mark it as ready to be used since
	// we're not going to modify it anymore
	ReleaseAsyncProperty(EStaticMeshAsyncProperties::RenderData);

	if( GetNumSourceModels() )
	{
		// Rescale simple collision if the user changed the mesh build scale
		GetBodySetup()->RescaleSimpleCollision( GetSourceModel(0).BuildSettings.BuildScale3D );
	}

	// Invalidate physics data if this has changed.
	// TODO_STATICMESH: Not necessary any longer?
	GetBodySetup()->InvalidatePhysicsData();
	GetBodySetup()->CreatePhysicsMeshes();

	// Update nav collision 
	CreateNavCollision(/*bIsUpdate=*/true);

	// Compare the derived data keys to see if renderable mesh data has actually changed.
	check(GetRenderData());
	bool bHasRenderDataChanged = GetRenderData()->DerivedDataKey != ExistingDerivedDataKey;

	if (bHasRenderDataChanged)
	{
		// Warn the user if the new mesh has degenerate tangent bases.
		bool bZeroNormals, bZeroTangents, bZeroBinormals;
		if (HasBadNTB(this, bZeroNormals, bZeroTangents, bZeroBinormals))
		{
			//Issue the tangent message in case tangent are zero
			if (bZeroTangents || bZeroBinormals)
			{
				const FStaticMeshSourceModel& SourceModelLOD0 = GetSourceModel(0);
				bool bIsUsingMikktSpace = SourceModelLOD0.BuildSettings.bUseMikkTSpace && (SourceModelLOD0.BuildSettings.bRecomputeTangents || SourceModelLOD0.BuildSettings.bRecomputeNormals);
				// Only suggest Recompute Tangents if the import hasn't already tried it
				FFormatNamedArguments Arguments;
				Arguments.Add(TEXT("Meshname"), FText::FromString(GetName()));
				Arguments.Add(TEXT("Options"), SourceModelLOD0.BuildSettings.bRecomputeTangents ? FText::GetEmpty() : LOCTEXT("MeshRecomputeTangents", "Consider enabling Recompute Tangents in the mesh's Build Settings."));
				Arguments.Add(TEXT("MikkTSpace"), bIsUsingMikktSpace ? LOCTEXT("MeshUseMikkTSpace", "MikkTSpace relies on tangent bases and may result in mesh corruption, consider disabling this option.") : FText::GetEmpty());
				const FText WarningMsg = FText::Format(LOCTEXT("MeshHasDegenerateTangents", "{Meshname} has degenerate tangent bases which will result in incorrect shading. {Options} {MikkTSpace}"), Arguments);
				//Automation and unattended log display instead of warning for tangents
				if (FApp::IsUnattended())
				{
					UE_LOGF(LogStaticMesh, Display, "%ls", *WarningMsg.ToString());
				}
				else
				{
					UE_LOGF(LogStaticMesh, Warning, "%ls", *WarningMsg.ToString());
				}

				if (!BuildParameters.bInSilent && BuildParameters.OutErrors)
				{
					BuildParameters.OutErrors->Add(WarningMsg);
				}
			}
		}
		
		FText ToleranceArgument = FText::FromString(TEXT("1E-4"));
		if (bZeroNormals)
		{
			FFormatNamedArguments Arguments;
			Arguments.Add(TEXT("Meshname"), FText::FromString(GetName()));
			Arguments.Add(TEXT("Tolerance"), ToleranceArgument);
			const FText WarningMsg = FText::Format(LOCTEXT("MeshHasSomeZeroNormals", "{Meshname} has some nearly zero normals which can create some issues. (Tolerance of {Tolerance})"), Arguments);
			//Automation and unattended log display instead of warning for normals
			if (FApp::IsUnattended())
			{
				UE_LOGF(LogStaticMesh, Display, "%ls", *WarningMsg.ToString());
			}
			else
			{
				UE_LOGF(LogStaticMesh, Warning, "%ls", *WarningMsg.ToString());
			}
			if (!BuildParameters.bInSilent && BuildParameters.OutErrors)
			{
				BuildParameters.OutErrors->Add(WarningMsg);
			}
		}

		if (bZeroTangents)
		{
			FFormatNamedArguments Arguments;
			Arguments.Add(TEXT("Meshname"), FText::FromString(GetName()));
			Arguments.Add(TEXT("Tolerance"), ToleranceArgument);
			const FText WarningMsg = FText::Format(LOCTEXT("MeshHasSomeZeroTangents", "{Meshname} has some nearly zero tangents which can create some issues. (Tolerance of {Tolerance})"), Arguments);
			//Automation and unattended log display instead of warning for tangents
			if (FApp::IsUnattended())
			{
				UE_LOGF(LogStaticMesh, Display, "%ls", *WarningMsg.ToString());
			}
			else
			{
				UE_LOGF(LogStaticMesh, Warning, "%ls", *WarningMsg.ToString());
			}

			if (!BuildParameters.bInSilent && BuildParameters.OutErrors)
			{
				BuildParameters.OutErrors->Add(WarningMsg);
			}
		}

		if (bZeroBinormals)
		{
			FFormatNamedArguments Arguments;
			Arguments.Add(TEXT("Meshname"), FText::FromString(GetName()));
			Arguments.Add(TEXT("Tolerance"), ToleranceArgument);
			const FText WarningMsg = FText::Format(LOCTEXT("MeshHasSomeZeroBiNormals", "{Meshname} has some nearly zero bi-normals which can create some issues. (Tolerance of {Tolerance})"), Arguments);
			//Automation and unattended log display instead of warning for tangents
			if (FApp::IsUnattended())
			{
				UE_LOGF(LogStaticMesh, Display, "%ls", *WarningMsg.ToString());
			}
			else
			{
				UE_LOGF(LogStaticMesh, Warning, "%ls", *WarningMsg.ToString());
			}

			if (!BuildParameters.bInSilent && BuildParameters.OutErrors)
			{
				BuildParameters.OutErrors->Add(WarningMsg);
			}
		}

		// Force the static mesh to re-export next time lighting is built
		SetLightingGuid();
	}

	return bHasRenderDataChanged;
}

void UStaticMesh::FinishBuildInternal(const TArray<IStaticMeshComponent*>& InAffectedComponents, bool bHasRenderDataChanged, bool bShouldComputeExtendedBounds)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UStaticMesh::PostBuildInternal);

	// Note: meshes can be built during automated importing.  We should not create resources in that case
	// as they will never be released when this object is deleted
	if (FApp::CanEverRender())
	{
		// Reinitialize the static mesh's resources.
		InitResources();
	}

	for (IStaticMeshComponent* Component : InAffectedComponents)
	{
		Component->OnMeshRebuild(bHasRenderDataChanged);
	}

	// If extended bounds were already calculated in the PreBuild step and are unlocked, 
	// we will only validate here to avoid modifying the value while another thread could read it.
	if (bShouldComputeExtendedBounds)
	{
		// Calculate extended bounds
		CalculateExtendedBounds();
	}
	
	// Async build is finished, we can unlock everything now.
	ReleaseAsyncProperty();

	PostMeshBuild.Broadcast(this);
}

#endif // #if WITH_EDITOR

/*------------------------------------------------------------------------------
	Remapping of painted vertex colors.
------------------------------------------------------------------------------*/

#if WITH_EDITOR
/** Helper struct for the mesh component vert position octree */
struct FStaticMeshComponentVertPosOctreeSemantics
{
	enum { MaxElementsPerLeaf = 16 };
	enum { MinInclusiveElementsPerNode = 7 };
	enum { MaxNodeDepth = 12 };

	typedef TInlineAllocator<MaxElementsPerLeaf> ElementAllocator;

	/**
	 * Get the bounding box of the provided octree element. In this case, the box
	 * is merely the point specified by the element.
	 *
	 * @param	Element	Octree element to get the bounding box for
	 *
	 * @return	Bounding box of the provided octree element
	 */
	FORCEINLINE static FBoxCenterAndExtent GetBoundingBox( const FPaintedVertex& Element )
	{
		return FBoxCenterAndExtent( Element.Position, FVector::ZeroVector );
	}

	/**
	 * Determine if two octree elements are equal
	 *
	 * @param	A	First octree element to check
	 * @param	B	Second octree element to check
	 *
	 * @return	true if both octree elements are equal, false if they are not
	 */
	FORCEINLINE static bool AreElementsEqual( const FPaintedVertex& A, const FPaintedVertex& B )
	{
		return ( A.Position == B.Position && A.Normal == B.Normal && A.Color == B.Color );
	}

	/** Ignored for this implementation */
	FORCEINLINE static void SetElementId( const FPaintedVertex& Element, FOctreeElementId2 Id )
	{
	}
};
typedef TOctree2<FPaintedVertex, FStaticMeshComponentVertPosOctreeSemantics> TSMCVertPosOctree;

void RemapPaintedVertexColors(const TArray<FPaintedVertex>& InPaintedVertices,
	const FColorVertexBuffer* InOverrideColors,
	const FPositionVertexBuffer& OldPositions,
	const FStaticMeshVertexBuffer& OldVertexBuffer,
	const FPositionVertexBuffer& NewPositions,
	const FStaticMeshVertexBuffer* OptionalVertexBuffer,
	TArray<FColor>& OutOverrideColors)
{
	// Find the extents formed by the cached vertex positions in order to optimize the octree used later
	FVector MinExtents(ForceInitToZero);
	FVector MaxExtents(ForceInitToZero);
	
	TArray<FPaintedVertex> PaintedVertices;
	FBox Bounds(ForceInitToZero);

	// Retrieve currently painted vertices
	if (InPaintedVertices.Num() > 0)
	{
		// In case we have retained the painted vertices we can just append
		PaintedVertices.Append(InPaintedVertices);
		
		for (const FPaintedVertex& Vertex : InPaintedVertices)
		{
			Bounds += Vertex.Position;
		}
	}
	else if ( InOverrideColors )
	{
		// Otherwise we have to retrieve the data from the override color and vertex buffers
		TArray<FColor> Colors;
		InOverrideColors->GetVertexColors(Colors);

		PaintedVertices.Reset(Colors.Num());
		FPaintedVertex PaintedVertex;
		for (int32 Index = 0; Index < Colors.Num(); ++Index)
		{
			PaintedVertex.Color = Colors[Index];
			PaintedVertex.Normal = (FVector4)OldVertexBuffer.VertexTangentZ(Index);
			PaintedVertex.Position = (FVector)OldPositions.VertexPosition(Index);
			Bounds += PaintedVertex.Position;

			PaintedVertices.Add(PaintedVertex);
		}
	}

	// Create an octree which spans the extreme extents of the old and new vertex positions in order to quickly query for the colors
	// of the new vertex positions
	for (int32 VertIndex = 0; VertIndex < (int32)NewPositions.GetNumVertices(); ++VertIndex)
	{
		Bounds += (FVector)NewPositions.VertexPosition(VertIndex);
	}

	TSMCVertPosOctree VertPosOctree( Bounds.GetCenter(), Bounds.GetExtent().GetMax() );

	// Add each old vertex to the octree
	for ( int32 PaintedVertexIndex = 0; PaintedVertexIndex < PaintedVertices.Num(); ++PaintedVertexIndex )
	{
		VertPosOctree.AddElement( PaintedVertices[ PaintedVertexIndex ] );
	}

	// Iterate over each new vertex position, attempting to find the old vertex it is closest to, applying
	// the color of the old vertex to the new position if possible.
	OutOverrideColors.Empty(NewPositions.GetNumVertices());
	TArray<FPaintedVertex> PointsToConsider;
	const float DistanceOverNormalThreshold = OptionalVertexBuffer ? UE_KINDA_SMALL_NUMBER : 0.0f;
	for ( uint32 NewVertIndex = 0; NewVertIndex < NewPositions.GetNumVertices(); ++NewVertIndex )
	{
		PointsToConsider.Reset();
		const FVector3f& CurPosition = NewPositions.VertexPosition( NewVertIndex );
		FVector CurNormal = FVector::ZeroVector;
		if (OptionalVertexBuffer)
		{
			CurNormal = FVector4(OptionalVertexBuffer->VertexTangentZ( NewVertIndex ));
		}

		// Iterate through the octree attempting to find the vertices closest to the current new point
		VertPosOctree.FindNearbyElements((FVector)CurPosition, [&PointsToConsider](const FPaintedVertex& Vertex)
		{
			PointsToConsider.Add(Vertex);
		});

		// If any points to consider were found, iterate over each and find which one is the closest to the new point 
		if ( PointsToConsider.Num() > 0 )
		{
			FPaintedVertex BestVertex = PointsToConsider[0];
			FVector BestVertexNormal = BestVertex.Normal;

			float BestDistanceSquared = ( BestVertex.Position - (FVector)CurPosition ).SizeSquared();
			float BestNormalDot = BestVertexNormal | CurNormal;

			for ( int32 ConsiderationIndex = 1; ConsiderationIndex < PointsToConsider.Num(); ++ConsiderationIndex )
			{
				FPaintedVertex& Vertex = PointsToConsider[ ConsiderationIndex ];
				FVector VertexNormal = Vertex.Normal;

				const float DistSqrd = ( Vertex.Position - (FVector)CurPosition ).SizeSquared();
				const float NormalDot = VertexNormal | CurNormal;
				if ( DistSqrd < BestDistanceSquared - DistanceOverNormalThreshold )
				{
					BestVertex = Vertex;
					BestDistanceSquared = DistSqrd;
					BestNormalDot = NormalDot;
				}
				else if ( OptionalVertexBuffer && DistSqrd < BestDistanceSquared + DistanceOverNormalThreshold && NormalDot > BestNormalDot )
				{
					BestVertex = Vertex;
					BestDistanceSquared = DistSqrd;
					BestNormalDot = NormalDot;
				}
			}

			OutOverrideColors.Add(BestVertex.Color);
		}
	}
}
#endif // #if WITH_EDITOR

/*------------------------------------------------------------------------------
	Conversion of legacy source data.
------------------------------------------------------------------------------*/

#if WITH_EDITOR

struct FStaticMeshTriangle
{
	FVector3f	Vertices[3];
	FVector2f	UVs[3][8];
	FColor		Colors[3];
	int32		MaterialIndex;
	int32		FragmentIndex;
	uint32		SmoothingMask;
	int32		NumUVs;

	FVector3f	TangentX[3]; // Tangent, U-direction
	FVector3f	TangentY[3]; // Binormal, V-direction
	FVector3f	TangentZ[3]; // Normal

	uint32		bOverrideTangentBasis;
	uint32		bExplicitNormals;
};

void UStaticMesh::FixupZeroTriangleSections()
{
	if (GetRenderData()->MaterialIndexToImportIndex.Num() > 0 && GetRenderData()->LODResources.Num())
	{
		TArray<int32> MaterialMap;
		FMeshSectionInfoMap NewSectionInfoMap;

		// Iterate over all sections of all LODs and identify all material indices that need to be remapped.
		for (int32 LODIndex = 0; LODIndex < GetRenderData()->LODResources.Num(); ++ LODIndex)
		{
			FStaticMeshLODResources& LOD = GetRenderData()->LODResources[LODIndex];
			int32 NumSections = LOD.Sections.Num();

			for (int32 SectionIndex = 0; SectionIndex < NumSections; ++SectionIndex)
			{
				FMeshSectionInfo DefaultSectionInfo(SectionIndex);
				if (GetRenderData()->MaterialIndexToImportIndex.IsValidIndex(SectionIndex))
				{
					int32 ImportIndex = GetRenderData()->MaterialIndexToImportIndex[SectionIndex];
					FMeshSectionInfo SectionInfo = GetSectionInfoMap().Get(LODIndex, ImportIndex);
					int32 OriginalMaterialIndex = SectionInfo.MaterialIndex;

					// If import index == material index, remap it.
					if (SectionInfo.MaterialIndex == ImportIndex)
					{
						SectionInfo.MaterialIndex = SectionIndex;
					}

					// Update the material mapping table.
					while (SectionInfo.MaterialIndex >= MaterialMap.Num())
					{
						MaterialMap.Add(INDEX_NONE);
					}
					if (SectionInfo.MaterialIndex >= 0)
					{
						MaterialMap[SectionInfo.MaterialIndex] = OriginalMaterialIndex;
					}

					// Update the new section info map if needed.
					if (SectionInfo != DefaultSectionInfo)
					{
						NewSectionInfoMap.Set(LODIndex, SectionIndex, SectionInfo);
					}
				}
			}
		}

		// Compact the materials array.
		for (int32 i = GetRenderData()->LODResources[0].Sections.Num(); i < MaterialMap.Num(); ++i)
		{
			if (MaterialMap[i] == INDEX_NONE)
			{
				int32 NextValidIndex = i+1;
				for (; NextValidIndex < MaterialMap.Num(); ++NextValidIndex)
				{
					if (MaterialMap[NextValidIndex] != INDEX_NONE)
					{
						break;
					}
				}
				if (MaterialMap.IsValidIndex(NextValidIndex))
				{
					MaterialMap[i] = MaterialMap[NextValidIndex];
					for (TMap<uint32,FMeshSectionInfo>::TIterator It(NewSectionInfoMap.Map); It; ++It)
					{
						FMeshSectionInfo& SectionInfo = It.Value();
						if (SectionInfo.MaterialIndex == NextValidIndex)
						{
							SectionInfo.MaterialIndex = i;
						}
					}
				}
				MaterialMap.RemoveAt(i, NextValidIndex - i);
			}
		}

		GetSectionInfoMap().Clear();
		GetSectionInfoMap().CopyFrom(NewSectionInfoMap);

		// Check if we need to remap materials.
		bool bRemapMaterials = false;
		for (int32 MaterialIndex = 0; MaterialIndex < MaterialMap.Num(); ++MaterialIndex)
		{
			if (MaterialMap[MaterialIndex] != MaterialIndex)
			{
				bRemapMaterials = true;
				break;
			}
		}

		// Remap the materials array if needed.
		if (bRemapMaterials)
		{
			TArray<FStaticMaterial> OldMaterials;
			Exchange(GetStaticMaterials(),OldMaterials);
			GetStaticMaterials().Empty(MaterialMap.Num());
			for (int32 MaterialIndex = 0; MaterialIndex < MaterialMap.Num(); ++MaterialIndex)
			{
				FStaticMaterial StaticMaterial;
				int32 OldMaterialIndex = MaterialMap[MaterialIndex];
				if (OldMaterials.IsValidIndex(OldMaterialIndex))
				{
					StaticMaterial = OldMaterials[OldMaterialIndex];
				}
				GetStaticMaterials().Add(StaticMaterial);
			}
		}
	}
	else
	{
		int32 FoundMaxMaterialIndex = -1;
		TSet<int32> DiscoveredMaterialIndices;
		
		// Find the maximum material index that is used by the mesh
		// Also keep track of which materials are actually used in the array
		for(int32 LODIndex = 0; LODIndex < GetRenderData()->LODResources.Num(); ++LODIndex)
		{
			if (GetRenderData()->LODResources.IsValidIndex(LODIndex))
			{
				FStaticMeshLODResources& LOD = GetRenderData()->LODResources[LODIndex];
				int32 NumSections = LOD.Sections.Num();
				for (int32 SectionIndex = 0; SectionIndex < NumSections; ++SectionIndex)
				{
					FMeshSectionInfo Info = GetSectionInfoMap().Get(LODIndex, SectionIndex);
					if(Info.MaterialIndex > FoundMaxMaterialIndex)
					{
						FoundMaxMaterialIndex = Info.MaterialIndex;
					}

					DiscoveredMaterialIndices.Add(Info.MaterialIndex);
				}
			}
		}

		// NULL references to materials in indices that are not used by any LOD.
		// This is to fix up an import bug which caused more materials to be added to this array than needed.
		for ( int32 MaterialIdx = 0; MaterialIdx < GetStaticMaterials().Num(); ++MaterialIdx )
		{
			if ( !DiscoveredMaterialIndices.Contains(MaterialIdx) )
			{
				// Materials that are not used by any LOD resource should not be in this array.
				GetStaticMaterials()[MaterialIdx].MaterialInterface = nullptr;
			}
		}

		// Remove entries at the end of the materials array.
		if (GetStaticMaterials().Num() > (FoundMaxMaterialIndex + 1))
		{
			GetStaticMaterials().RemoveAt(FoundMaxMaterialIndex+1, GetStaticMaterials().Num() - FoundMaxMaterialIndex - 1);
		}
	}
}

#endif // #if WITH_EDITOR

#undef LOCTEXT_NAMESPACE
