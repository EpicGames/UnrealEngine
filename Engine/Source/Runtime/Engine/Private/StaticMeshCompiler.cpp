// Copyright Epic Games, Inc. All Rights Reserved.

#include "StaticMeshCompiler.h"

#if WITH_EDITOR

#include "AsyncCompilationHelpers.h"
#include "AssetCompilingManager.h"
#include "Algo/NoneOf.h"
#include "Components/StaticMeshComponent.h"
#include "Engine/StaticMesh.h"
#include "Editor.h"
#include "ObjectCacheContext.h"
#include "EngineLogs.h"
#include "Settings/EditorExperimentalSettings.h"
#include "GameFramework/Pawn.h"
#include "Logging/StructuredLog.h"
#include "Misc/QueuedThreadPoolWrapper.h"
#include "Templates/GuardValueAccessors.h"
#include "ProfilingDebugging/CountersTrace.h"
#include "TextureCompiler.h"
#include "StaticMeshResources.h"
#include "ShaderCompiler.h"
#include "ContentStreaming.h"
#include "EngineUtils.h"

#define LOCTEXT_NAMESPACE "StaticMeshCompiler"

static AsyncCompilationHelpers::FAsyncCompilationStandardCVars CVarAsyncStaticMeshStandard(
	TEXT("StaticMesh"),
	TEXT("static meshes"),
	FConsoleCommandDelegate::CreateLambda(
		[]()
		{
			FStaticMeshCompilingManager::Get().FinishAllCompilation();
		}
	));

static TAutoConsoleVariable<int32> CVarAsyncStaticMeshPlayInEditorMode(
	TEXT("Editor.AsyncStaticMeshPlayInEditorMode"),
	0,
	TEXT("0 - Wait until all static meshes are built before entering PIE. (Slowest but causes no visual or behavior artifacts.) \n")
	TEXT("1 - Wait until all static meshes affecting navigation and physics are built before entering PIE. (Some visuals might be missing during compilation.)\n")
	TEXT("2 - Wait only on static meshes affecting navigation and physics when they are close to the player. (Fastest while still preventing falling through the floor and going through objects.)\n"),
	ECVF_Default);

static TAutoConsoleVariable<float> CVarAsyncStaticMeshPlayInEditorDistance(
	TEXT("Editor.AsyncStaticMeshPlayInEditorDistance"),
	2.0f,
	TEXT("Scale applied to the player bounding sphere to determine how far away to force meshes compilation before resuming play.\n")
	TEXT("The effect can be seen during play session when Editor.AsyncStaticMeshPlayInEditorDebugDraw = 1.\n"),
	ECVF_Default);

static TAutoConsoleVariable<bool> CVarAsyncStaticMeshDebugDraw(
	TEXT("Editor.AsyncStaticMeshPlayInEditorDebugDraw"),
	false,
	TEXT("0 - Debug draw for async static mesh compilation is disabled.\n")
	TEXT("1 - Debug draw for async static mesh compilation is enabled.\n")
	TEXT("The collision sphere around the player is drawn in white and can be adjusted with Editor.AsyncStaticMeshPlayInEditorDistance\n")
	TEXT("Any static meshes affecting the physics that are still being compiled will have their bounding box drawn in green.\n")
	TEXT("Any static meshes that were waited on due to being too close to the player will have their bounding box drawn in red for a couple of seconds."),
	ECVF_Default);

static bool GAsyncStaticMeshCompilationCancelable = true;
static FAutoConsoleVariableRef CVarAsyncStaticMeshCompilationCancelable(
	TEXT("Editor.AsyncStaticMeshCompilationCancelable"),
	GAsyncStaticMeshCompilationCancelable,
	TEXT("Whether or not to allow early cancelation of static meshes during async compilation."),
	ECVF_Default);

namespace StaticMeshCompilingManagerImpl
{
	static void EnsureInitializedCVars()
	{
		static bool bIsInitialized = false;

		if (!bIsInitialized)
		{
			bIsInitialized = true;

			AsyncCompilationHelpers::EnsureInitializedCVars(
				TEXT("staticmesh"),
				CVarAsyncStaticMeshStandard.AsyncCompilation,
				CVarAsyncStaticMeshStandard.AsyncCompilationMaxConcurrency,
				GET_MEMBER_NAME_CHECKED(UEditorExperimentalSettings, bEnableAsyncStaticMeshCompilation));
		}
	}
}

FStaticMeshCompilingManager::FStaticMeshCompilingManager()
	: Notification(MakeUnique<FAsyncCompilationNotification>(GetAssetNameFormat()))
{
	StaticMeshCompilingManagerImpl::EnsureInitializedCVars();

	PostReachabilityAnalysisHandle = FCoreUObjectDelegates::PostReachabilityAnalysis.AddRaw(this, &FStaticMeshCompilingManager::OnPostReachabilityAnalysis);
}

bool FStaticMeshCompilingManager::IsAsyncCompilationCancelable() const
{
	return GAsyncStaticMeshCompilationCancelable;
}

void FStaticMeshCompilingManager::MarkCompilationAsCanceled(TArrayView<UObject* const> InObjects)
{
	if (GetNumRemainingMeshes() && InObjects.Num())
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FStaticMeshCompilingManager::MarkCompilationAsCanceled);

		TSet<UStaticMesh*> StaticMeshes;
		for (UObject* Object : InObjects)
		{
			if (UStaticMesh* StaticMesh = Cast<UStaticMesh>(Object))
			{
				StaticMeshes.Add(StaticMesh);
			}
		}

		if (StaticMeshes.Num())
		{
			auto CancelOrMarkCanceled = [&StaticMeshes](TSet<TWeakObjectPtr<UStaticMesh>>& Set)
			{
				for (auto Iterator = Set.CreateIterator(); Iterator; ++Iterator)
				{
					UStaticMesh* StaticMesh = Iterator->GetEvenIfUnreachable();
					if (StaticMesh && StaticMeshes.Contains(StaticMesh))
					{
						UE_LOGF(LogStaticMesh, Verbose, "Canceling static mesh %ls async compilation as requested", *StaticMesh->GetName());

						// On success, we can remove them from the list right away.
						// Otherwise, they are marked as canceled and will finish ASAP if the tasks support early cancellation.
						if (StaticMesh->TryCancelAsyncTasks())
						{
							Iterator.RemoveCurrent();
						}
					}
				}
			};

			CancelOrMarkCanceled(RegisteredStaticMesh);
			CancelOrMarkCanceled(StaticMeshesWithPendingDependencies);
		}
	}
}

void FStaticMeshCompilingManager::OnPostReachabilityAnalysis()
{
	if (GetNumRemainingMeshes())
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FStaticMeshCompilingManager::CancelUnreachableMeshes);

		TArray<UStaticMesh*> PendingStaticMeshes;
		PendingStaticMeshes.Reserve(GetNumRemainingMeshes());

		auto CancelOrCollectUnreachable = [&PendingStaticMeshes](TSet<TWeakObjectPtr<UStaticMesh>>& Set)
		{
			for (auto Iterator = Set.CreateIterator(); Iterator; ++Iterator)
			{
				UStaticMesh* StaticMesh = Iterator->GetEvenIfUnreachable();
				if (StaticMesh && StaticMesh->IsUnreachable())
				{
					UE_LOGF(LogStaticMesh, Verbose, "Cancelling static mesh %ls async compilation because it's being garbage collected", *StaticMesh->GetName());

					if (StaticMesh->TryCancelAsyncTasks())
					{
						Iterator.RemoveCurrent();
					}
					else
					{
						PendingStaticMeshes.Add(StaticMesh);
					}
				}
			}
		};

		CancelOrCollectUnreachable(RegisteredStaticMesh);
		CancelOrCollectUnreachable(StaticMeshesWithPendingDependencies);

		if (!PendingStaticMeshes.IsEmpty())
		{
			FinishCompilation(PendingStaticMeshes);
		}
	}
}

FName FStaticMeshCompilingManager::GetStaticAssetTypeName()
{
	return TEXT("UE-StaticMesh");
}

FName FStaticMeshCompilingManager::GetAssetTypeName() const
{
	return GetStaticAssetTypeName();
}

FTextFormat FStaticMeshCompilingManager::GetAssetNameFormat() const
{
	return LOCTEXT("StaticMeshNameFormat", "{0}|plural(one=Static Mesh,other=Static Meshes)");
}

TArrayView<FName> FStaticMeshCompilingManager::GetDependentTypeNames() const
{
	// Texture and shaders can affect materials which can affect Static Meshes once they are visible.
	// Adding these dependencies can reduces the actual number of render state update we need to do in a frame
	static FName DependentTypeNames[] = 
	{ 
		FTextureCompilingManager::GetStaticAssetTypeName(),
		FShaderCompilingManager::GetStaticAssetTypeName()
	};
	return TArrayView<FName>(DependentTypeNames);
}

EQueuedWorkPriority FStaticMeshCompilingManager::GetBasePriority(UStaticMesh* InStaticMesh) const
{
	return EQueuedWorkPriority::Low;
}

FQueuedThreadPool* FStaticMeshCompilingManager::GetThreadPool() const
{
	static FQueuedThreadPoolDynamicWrapper* GStaticMeshThreadPool = nullptr;
	if (GStaticMeshThreadPool == nullptr && FAssetCompilingManager::Get().GetThreadPool() != nullptr)
	{
		// Static meshes will be scheduled on the asset thread pool, where concurrency limits might by dynamically adjusted depending on memory constraints.
		GStaticMeshThreadPool = new FQueuedThreadPoolDynamicWrapper(FAssetCompilingManager::Get().GetThreadPool(), -1, [](EQueuedWorkPriority) { return EQueuedWorkPriority::Low; });

		AsyncCompilationHelpers::BindThreadPoolToCVar(
			GStaticMeshThreadPool,
			CVarAsyncStaticMeshStandard.AsyncCompilation,
			CVarAsyncStaticMeshStandard.AsyncCompilationResume,
			CVarAsyncStaticMeshStandard.AsyncCompilationMaxConcurrency
		);
	}

	return GStaticMeshThreadPool;
}

void FStaticMeshCompilingManager::Shutdown()
{
	bHasShutdown = true;
	if (GetNumRemainingMeshes())
	{
		check(IsInGameThread());
		TRACE_CPUPROFILER_EVENT_SCOPE(FStaticMeshCompilingManager::Shutdown)

		TArray<UStaticMesh*> PendingStaticMeshes;
		PendingStaticMeshes.Reserve(GetNumRemainingMeshes());

		auto CancelOrCollect = [&PendingStaticMeshes] (TSet<TWeakObjectPtr<UStaticMesh>>& Set)
		{
			for (TWeakObjectPtr<UStaticMesh>& WeakStaticMesh : Set)
			{
				if (WeakStaticMesh.IsValid())
				{
					UStaticMesh* StaticMesh = WeakStaticMesh.Get();
					if (!StaticMesh->TryCancelAsyncTasks())
					{
						PendingStaticMeshes.Add(StaticMesh);
					}
				}
			}
		};

		CancelOrCollect(RegisteredStaticMesh);
		CancelOrCollect(StaticMeshesWithPendingDependencies);

		if (!PendingStaticMeshes.IsEmpty())
		{
			FinishCompilation(PendingStaticMeshes);
		}
	}

	FCoreUObjectDelegates::PostReachabilityAnalysis.Remove(PostReachabilityAnalysisHandle);
}

bool FStaticMeshCompilingManager::IsAsyncStaticMeshCompilationEnabled() const
{
	if (bHasShutdown || !FPlatformProcess::SupportsMultithreading())
	{
		return false;
	}

	return CVarAsyncStaticMeshStandard.AsyncCompilation.GetValueOnAnyThread() != 0;
}

TRACE_DECLARE_INT_COUNTER(QueuedStaticMeshCompilation, TEXT("AsyncCompilation/QueuedStaticMesh"));
void FStaticMeshCompilingManager::UpdateCompilationNotification()
{
	TRACE_COUNTER_SET(QueuedStaticMeshCompilation, GetNumRemainingMeshes());
	Notification->Update(GetNumRemainingMeshes());
}

void FStaticMeshCompilingManager::PostCompilation(TArrayView<UStaticMesh* const> InStaticMeshes)
{
	if (InStaticMeshes.Num())
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(OnAssetPostCompileEvent);

		TArray<FAssetCompileData> AssetsData;
		AssetsData.Reserve(InStaticMeshes.Num());

		for (UStaticMesh* StaticMesh : InStaticMeshes)
		{
			// Do not broadcast an event for unreachable objects
			if (!StaticMesh->IsUnreachable())
			{
				AssetsData.Emplace(StaticMesh);
			}
		}

		if (AssetsData.Num())
		{
			FAssetCompilingManager::Get().OnAssetPostCompileEvent().Broadcast(AssetsData);
		}

		// Schedule compilations that were dependent upon others
		SchedulePendingCompilations();
	}
}

void FStaticMeshCompilingManager::PostCompilation(UStaticMesh* StaticMesh)
{
	using namespace StaticMeshCompilingManagerImpl;
	
	UE_LOG_CONTEXT("StaticMesh", FSoftObjectPath(StaticMesh));
	
	// If AsyncTask is null here, the task got canceled so we don't need to do anything
	if (StaticMesh->AsyncTask && !IsEngineExitRequested())
	{
		check(IsInGameThread());
		TRACE_CPUPROFILER_EVENT_SCOPE(PostCompilation);

		FObjectCacheContextScope ObjectCacheScope;

		// If async (post load or build), restore the state of GIsEditorLoadingPackage for the duration of this function (including outside of this scope) as it was when the build of the static mesh was initiated, 
		//  so that async builds have the same result as synchronous ones (e.g. don't dirty packages when the components referencing this static mesh call Modify because GIsEditorLoadingPackage is true) :
		TUniquePtr<TGuardValueAccessors<bool>> IsEditorLoadingPackageGuard;

		// The scope is important here to destroy the FStaticMeshAsyncBuildScope before broadcasting events
		{
			// Acquire the async task locally to protect against re-entrance
			TUniquePtr<FStaticMeshAsyncBuildTask> LocalAsyncTask = MoveTemp(StaticMesh->AsyncTask);
			LocalAsyncTask->EnsureCompletion();

			const bool bWasCanceled = 
				LocalAsyncTask->GetTask().BuildContext.IsValid() ? LocalAsyncTask->GetTask().BuildContext->CancellationToken.IsCanceled() : false ||
				LocalAsyncTask->GetTask().PostLoadContext.IsValid() ? LocalAsyncTask->GetTask().PostLoadContext->CancellationToken.IsCanceled() : false;

			// if it has dependencies, remove it from each dependent's reverse lookup
			for (UStaticMesh* Dependency : StaticMesh->GetCachedNaniteAssemblyReferences())
			{
				if (Dependency)
				{
					if (TSet<TWeakObjectPtr<UStaticMesh>>* ReverseDeps = ReverseDependencyLookup.Find(Dependency))
					{
						ReverseDeps->Remove(StaticMesh);
						if (ReverseDeps->IsEmpty())
						{
							ReverseDependencyLookup.Remove(Dependency);
						}
					}
				}
			}

			// Do not do anything else if the staticmesh is being garbage collected or has been canceled
			if (bWasCanceled || StaticMesh->IsUnreachable())
			{
				{
					FStaticMeshAsyncBuildScope AsyncBuildScope(StaticMesh);
					// Invalidate the render data to make sure nothing tries to access it as it might 
					// be in an invalid state after cancelation.
					StaticMesh->SetRenderData(nullptr);
					StaticMesh->ReleaseAsyncProperty();
				}

				// Notify other systems that this static mesh has been canceled.
				// This might be a redundant call but the first one might have raced with distance field and 
				// mesh card tasks that can be added from worker threads.
				FAssetCompilingManager::Get().MarkCompilationAsCanceled({ StaticMesh });

				return;
			}

			UE_LOGF(LogStaticMesh, Verbose, "Refreshing static mesh %ls because it is ready", *StaticMesh->GetName());

			FStaticMeshAsyncBuildScope AsyncBuildScope(StaticMesh);

			if (LocalAsyncTask->GetTask().PostLoadContext.IsValid())
			{
				IsEditorLoadingPackageGuard.Reset(new TGuardValueAccessors<bool>(UE::GetIsEditorLoadingPackage, UE::SetIsEditorLoadingPackage, LocalAsyncTask->GetTask().PostLoadContext->bIsEditorLoadingPackage));

				StaticMesh->FinishPostLoadInternal(*LocalAsyncTask->GetTask().PostLoadContext);

				LocalAsyncTask->GetTask().PostLoadContext.Reset();
			}

			if (LocalAsyncTask->GetTask().BuildContext.IsValid())
			{
				IsEditorLoadingPackageGuard.Reset(new TGuardValueAccessors<bool>(UE::GetIsEditorLoadingPackage, UE::SetIsEditorLoadingPackage, LocalAsyncTask->GetTask().BuildContext->bIsEditorLoadingPackage));

				TArray<IStaticMeshComponent*> ComponentsToUpdate;
				for (IStaticMeshComponent* Component : ObjectCacheScope.GetContext().GetStaticMeshComponents(StaticMesh))
				{
					ComponentsToUpdate.Add(Component);
				}

				StaticMesh->FinishBuildInternal(
					ComponentsToUpdate,
					LocalAsyncTask->GetTask().BuildContext->bHasRenderDataChanged,
					LocalAsyncTask->GetTask().BuildContext->bShouldComputeExtendedBounds
				);

				LocalAsyncTask->GetTask().BuildContext.Reset();
			}
		}

		for (IStaticMeshComponent* Component : ObjectCacheScope.GetContext().GetStaticMeshComponents(StaticMesh))
		{
			Component->PostStaticMeshCompilation();
		}

		// Calling this delegate during app exit might be quite dangerous and lead to crash
		// if the content browser wants to refresh a thumbnail it might try to load a package
		// which will then fail due to various reasons related to the editor shutting down.
		// Triggering this callback while garbage collecting can also result in listeners trying to look up objects
		if (!GExitPurge && !IsGarbageCollecting())
		{
			// Generate an empty property changed event, to force the asset registry tag
			// to be refreshed now that RenderData is available.
			FPropertyChangedEvent EmptyPropertyChangedEvent(nullptr);
			FCoreUObjectDelegates::OnObjectPropertyChanged.Broadcast(StaticMesh, EmptyPropertyChangedEvent);
		}
	}
}

void FStaticMeshCompilingManager::SchedulePendingCompilations()
{
	TArray<UStaticMesh*> ReadyToSchedule;
	for (auto It = StaticMeshesWithPendingDependencies.CreateIterator(); It; ++It)
	{
		if (UStaticMesh* StaticMesh = It->Get())
		{
			if (StaticMesh->HasAnyDependenciesCompiling())
			{
				continue;
			}
			ReadyToSchedule.Emplace(StaticMesh);
		}
		It.RemoveCurrent();
	}

	if (ReadyToSchedule.Num() == 0)
	{
		return;
	}

	for (UStaticMesh* StaticMesh : ReadyToSchedule)
	{
		// The mesh's task hasn't been kicked yet, so start it now
		check(StaticMesh->AsyncTask != nullptr && StaticMesh->AsyncTask->IsIdle());
		int64 BuildRequiredMemory = 0;
		{
			// Get access to source meshes without causing a stall here.
			// NOTE: It's only safe to do this because we know the task has not been started
			FStaticMeshAsyncBuildScope AsyncBuildScope(StaticMesh);
			BuildRequiredMemory = StaticMesh->GetBuildRequiredMemoryEstimate();
		}
		StaticMesh->AsyncTask->StartBackgroundTask(
			GetThreadPool(),
			GetBasePriority(StaticMesh),
			EQueuedWorkFlags::DoNotRunInsideBusyWait,
			BuildRequiredMemory,
			TEXT("StaticMesh")
		);
	}

	// Add the static meshes that are now in progress
	AddStaticMeshes(ReadyToSchedule);
}

bool FStaticMeshCompilingManager::IsAsyncCompilationAllowed(UStaticMesh* StaticMesh) const
{
	return IsAsyncStaticMeshCompilationEnabled();
}

FStaticMeshCompilingManager& FStaticMeshCompilingManager::Get()
{
	static FStaticMeshCompilingManager Singleton;
	return Singleton;
}

int32 FStaticMeshCompilingManager::GetNumRemainingMeshes() const
{
	return RegisteredStaticMesh.Num() + StaticMeshesWithPendingDependencies.Num();
}

int32 FStaticMeshCompilingManager::GetNumRemainingAssets() const
{
	return GetNumRemainingMeshes();
}

void FStaticMeshCompilingManager::AddStaticMeshes(TArrayView<UStaticMesh* const> InStaticMeshes)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FStaticMeshCompilingManager::AddStaticMeshes)
	check(IsInGameThread());

	// Wait until we gather enough mesh to process
	// to amortize the cost of scanning components
	//ProcessStaticMeshes(32 /* MinBatchSize */);

	for (UStaticMesh* StaticMesh : InStaticMeshes)
	{
		check(StaticMesh->AsyncTask != nullptr);
		RegisteredStaticMesh.Emplace(StaticMesh);
	}

	TRACE_COUNTER_SET(QueuedStaticMeshCompilation, GetNumRemainingMeshes());
}

void FStaticMeshCompilingManager::AddStaticMeshesWithDependencies(TArrayView<UStaticMesh* const> InStaticMeshes)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FStaticMeshCompilingManager::AddStaticMeshes)
	check(IsInGameThread());

	for (UStaticMesh* StaticMesh : InStaticMeshes)
	{
		for (UStaticMesh* Dependency : StaticMesh->GetCachedNaniteAssemblyReferences())
		{
			if (Dependency)
			{
				TSet<TWeakObjectPtr<UStaticMesh>>& ReverseLookupSet = ReverseDependencyLookup.FindOrAdd(Dependency);
				ReverseLookupSet.Add(StaticMesh);
			}
		}

		check(StaticMesh->AsyncTask != nullptr);
		if (StaticMesh->AsyncTask->IsIdle())
		{
			// Async task couldn't be started yet, put it in the pending queue
			StaticMeshesWithPendingDependencies.Emplace(StaticMesh);
		}
		else
		{
			RegisteredStaticMesh.Emplace(StaticMesh);
		}
	}

	TRACE_COUNTER_SET(QueuedStaticMeshCompilation, GetNumRemainingMeshes());
}

void FStaticMeshCompilingManager::FinishCompilation(TArrayView<UStaticMesh* const> InStaticMeshes, const FFinishCompilationOptions& Options)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FStaticMeshCompilingManager::FinishCompilation);

	// Allow calls from any thread if the meshes are already finished compiling.
	if (!Options.bIncludeDependentMeshes && Algo::NoneOf(InStaticMeshes, &UStaticMesh::IsCompiling))
	{
		return;
	}

	check(IsInGameThread());

	TArray<UStaticMesh*> PendingStaticMeshes, NextPendingStaticMeshes;
	PendingStaticMeshes.Reserve(InStaticMeshes.Num());

	auto FilterIntoPendingLists =
		[this, &PendingStaticMeshes, &NextPendingStaticMeshes, &Options] (TArrayView<UStaticMesh* const> Meshes)
	{
		PendingStaticMeshes.SetNum(0, EAllowShrinking::No);
		NextPendingStaticMeshes.SetNum(0, EAllowShrinking::No);

		int32 StaticMeshIndex = 0;
		for (UStaticMesh* StaticMesh : Meshes)
		{
			if (RegisteredStaticMesh.Contains(StaticMesh))
			{
				PendingStaticMeshes.AddUnique(StaticMesh);
			}
			else if (StaticMeshesWithPendingDependencies.Contains(StaticMesh))
			{
				// Add it to the next wave of meshes to finish
				NextPendingStaticMeshes.AddUnique(StaticMesh);
			}

			if (Options.bIncludeDependentMeshes)
			{
				// If we're stalling on the mesh compilation because we're about to edit the mesh, we have to stall on any
				// compiling mesh that depends on us as well, to make sure we don't write to the mesh while it's being read
				if (TSet<TWeakObjectPtr<UStaticMesh>>* ReverseLookup = ReverseDependencyLookup.Find(StaticMesh))
				{
					for (TWeakObjectPtr<UStaticMesh>& ReverseDependency : *ReverseLookup)
					{
						if (ReverseDependency.IsValid())
						{
							NextPendingStaticMeshes.AddUnique(ReverseDependency.Get());
						}
					}
				}
			}
		}

		// Recursively filter in all dependencies
		for (int32 NextPendingIndex = 0; NextPendingIndex < NextPendingStaticMeshes.Num(); ++NextPendingIndex)
		{
			UStaticMesh* StaticMesh = NextPendingStaticMeshes[NextPendingIndex];
			for (UStaticMesh* Dependency : StaticMesh->GetCachedNaniteAssemblyReferences())
			{
				if (RegisteredStaticMesh.Contains(Dependency))
				{
					PendingStaticMeshes.AddUnique(Dependency);
				}
				else if (StaticMeshesWithPendingDependencies.Contains(Dependency))
				{
					NextPendingStaticMeshes.AddUnique(Dependency);
				}
			}
		}

		if (PendingStaticMeshes.Num() == 0 && NextPendingStaticMeshes.Num() > 0)
		{
			PendingStaticMeshes = MoveTemp(NextPendingStaticMeshes);
		}
	};

	FilterIntoPendingLists(InStaticMeshes);
	while (PendingStaticMeshes.Num())
	{
		class FCompilableStaticMesh : public AsyncCompilationHelpers::TCompilableAsyncTask<FStaticMeshAsyncBuildTask>
		{
		public:
			FCompilableStaticMesh(UStaticMesh* InStaticMesh)
				: StaticMesh(InStaticMesh)
			{
			}

			FStaticMeshAsyncBuildTask* GetAsyncTask() override
			{
				return StaticMesh->AsyncTask.Get();
			}

			UStaticMesh* StaticMesh;
			FName GetName() override { return StaticMesh->GetOutermost()->GetFName(); }
		};

		TArray<FCompilableStaticMesh> CompilableStaticMeshes(PendingStaticMeshes);
		FObjectCacheContextScope ObjectCacheScope;
		AsyncCompilationHelpers::FinishCompilation(
			[&CompilableStaticMeshes](int32 Index)	-> AsyncCompilationHelpers::ICompilable& { return CompilableStaticMeshes[Index]; },
			CompilableStaticMeshes.Num(),
			LOCTEXT("StaticMeshes", "Static Meshes"),
			LogStaticMesh,
			[this](AsyncCompilationHelpers::ICompilable* Object)
			{
				UStaticMesh* StaticMesh = static_cast<FCompilableStaticMesh*>(Object)->StaticMesh;
				PostCompilation(StaticMesh);
				RegisteredStaticMesh.Remove(StaticMesh);
			}
		);

		PostCompilation(PendingStaticMeshes);

		TArray<UStaticMesh*> Temp = MoveTemp(NextPendingStaticMeshes);
		FilterIntoPendingLists(Temp);
	}

	// Sanity check - if no dependencies are pending, it should have already been put in the active list
	check(NextPendingStaticMeshes.Num() == 0);
}

void FStaticMeshCompilingManager::FinishCompilationsForGame()
{
	if (GetNumRemainingMeshes())
	{
		FObjectCacheContextScope ObjectCacheScope;
		// Supports both Game and PIE mode
		const bool bIsPlaying = 
			(GWorld && !GWorld->IsEditorWorld()) ||
			(GEditor && GEditor->PlayWorld && !GEditor->IsSimulateInEditorInProgress());

		if (bIsPlaying)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(FStaticMeshCompilingManager::FinishCompilationsForGame);

			const int32 PlayInEditorMode = CVarAsyncStaticMeshPlayInEditorMode.GetValueOnGameThread();
			
			const bool bShowDebugDraw = CVarAsyncStaticMeshDebugDraw.GetValueOnGameThread();

			TSet<const UWorld*> PIEWorlds;
			TMultiMap<const UWorld*, FBoxSphereBounds> WorldActors;
			
			float RadiusScale = CVarAsyncStaticMeshPlayInEditorDistance.GetValueOnGameThread();
			for (const FWorldContext& WorldContext : GEngine->GetWorldContexts())
			{
				if (WorldContext.WorldType == EWorldType::PIE || WorldContext.WorldType == EWorldType::Game)
				{
					UWorld* World = WorldContext.World();
					PIEWorlds.Add(World);

					// Extract all pawns of the world to support player/bots local and remote.
					if (PlayInEditorMode == 2)
					{
						for (TActorIterator<APawn> It(World); It; ++It)
						{
							const APawn* Pawn = *It;
							if (Pawn)
							{
								FBoxSphereBounds ActorBounds;
								Pawn->GetActorBounds(true, ActorBounds.Origin, ActorBounds.BoxExtent);
								ActorBounds.SphereRadius = ActorBounds.BoxExtent.GetMax() * RadiusScale;
								WorldActors.Emplace(World, ActorBounds);

								if (bShowDebugDraw)
								{
									DrawDebugSphere(World, ActorBounds.Origin, ActorBounds.SphereRadius, 10, FColor::White);
								}
							}
						}
					}
				}
			}
			
			TSet<UStaticMesh*> StaticMeshToCompile;
			TArray<FBoxSphereBounds, TInlineAllocator<16>> ActorsBounds;
			for (TWeakObjectPtr<UStaticMesh>& StaticMeshPtr : RegisteredStaticMesh)
			{
				if (UStaticMesh* StaticMesh = StaticMeshPtr.Get())
				{
					for (const IStaticMeshComponent* ComponentInterface : ObjectCacheScope.GetContext().GetStaticMeshComponents(StaticMesh))
					{							
						const IPrimitiveComponent* PrimComponentInterface = ComponentInterface->GetPrimitiveComponentInterface();
						const UPrimitiveComponent* PrimComponent = PrimComponentInterface->GetUObject<UPrimitiveComponent>();
						bool bHasRelevantCollision = !PrimComponent || (PrimComponent->GetCollisionEnabled() != ECollisionEnabled::NoCollision || PrimComponent->IsNavigationRelevant() || PrimComponent->bAlwaysCreatePhysicsState || PrimComponent->CanCharacterStepUpOn != ECB_No);

						if (PIEWorlds.Contains(PrimComponentInterface->GetWorld()) &&
							(PlayInEditorMode == 0 || bHasRelevantCollision))
						{
							if (PlayInEditorMode == 2)
							{
								const FBoxSphereBounds ComponentBounds = PrimComponentInterface->GetBounds().GetBox();
								const UWorld* ComponentWorld = PrimComponentInterface->GetWorld();

								ActorsBounds.Reset();
								WorldActors.MultiFind(ComponentWorld, ActorsBounds);
						
								bool bStaticMeshComponentCollided = false;
								if (ActorsBounds.Num())
								{
									for (const FBoxSphereBounds& ActorBounds : ActorsBounds)
									{
										if (FMath::SphereAABBIntersection(ActorBounds.Origin, ActorBounds.SphereRadius * ActorBounds.SphereRadius, ComponentBounds.GetBox()))
										{
											if (bShowDebugDraw)
											{
												DrawDebugBox(ComponentWorld, ComponentBounds.Origin, ComponentBounds.BoxExtent, FColor::Red, false, 10.0f);
											}
								
											bool bIsAlreadyInSet = false;
											StaticMeshToCompile.Add(ComponentInterface->GetStaticMesh(), &bIsAlreadyInSet);
											if (!bIsAlreadyInSet)
											{
												UE_LOGF(
													LogStaticMesh,
													Display,
													"Waiting on static mesh %ls being ready because it affects collision/navigation and is near a player/bot",
													*ComponentInterface->GetStaticMesh()->GetFullName()
												);
											}
											bStaticMeshComponentCollided = true;
											break;
										}
									}
								}

								if (bShowDebugDraw && !bStaticMeshComponentCollided)
								{
									DrawDebugBox(ComponentWorld, ComponentBounds.Origin, ComponentBounds.BoxExtent, FColor::Green);
								}

								// No need to iterate throught all components once we have found one that requires the static mesh to finish compilation 
								// unless bShowDebugDraw is activated.
								if (!bShowDebugDraw)
								{
									break;
								}
							}
							else 
							{
								bool bIsAlreadyInSet = false;
								StaticMeshToCompile.Add(StaticMesh, &bIsAlreadyInSet);
								if (!bIsAlreadyInSet)
								{
									if (PlayInEditorMode == 0)
									{
										UE_LOGF(LogStaticMesh, Display, "Waiting on static mesh %ls being ready before playing", *StaticMesh->GetFullName());
									}
									else
									{
										UE_LOGF(LogStaticMesh, Display, "Waiting on static mesh %ls being ready because it affects collision/navigation", *StaticMesh->GetFullName());
									}
								}

								// No need to iterate throught all components once we have found one that requires the static mesh to finish compilation.
								break;
							}
						}
					}
				}
			}

			if (StaticMeshToCompile.Num())
			{
				FinishCompilation(StaticMeshToCompile.Array());
			}
		}
	}
}

void FStaticMeshCompilingManager::FinishAllCompilation()
{
	check(IsInGameThread());
	TRACE_CPUPROFILER_EVENT_SCOPE(FStaticMeshCompilingManager::FinishAllCompilation)

	int32 NumRemainingMeshes = GetNumRemainingMeshes();
	if (NumRemainingMeshes)
	{
		TArray<UStaticMesh*> PendingStaticMeshes;
		PendingStaticMeshes.Reserve(NumRemainingMeshes);

		auto CollectAllValid = [&PendingStaticMeshes](TSet<TWeakObjectPtr<UStaticMesh>>& Set)
		{
			for (TWeakObjectPtr<UStaticMesh>& StaticMesh : Set)
			{
				if (StaticMesh.IsValid())
				{
					PendingStaticMeshes.Add(StaticMesh.Get());
				}
			}
		};

		CollectAllValid(RegisteredStaticMesh);
		CollectAllValid(StaticMeshesWithPendingDependencies);

		if (!PendingStaticMeshes.IsEmpty())
		{
			FinishCompilation(PendingStaticMeshes);
		}
	}
}

void FStaticMeshCompilingManager::FinishCompilationForObjects(TArrayView<UObject* const> InObjects)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FStaticMeshCompilingManager::FinishCompilationForObjects);

	TSet<UStaticMesh*> StaticMeshes;
	for (UObject* Object : InObjects)
	{
		if (UStaticMesh* StaticMesh = Cast<UStaticMesh>(Object))
		{
			StaticMeshes.Add(StaticMesh);
		}
		else if (UStaticMeshComponent* StaticMeshComponent = Cast<UStaticMeshComponent>(Object))
		{
			if (StaticMeshComponent->GetStaticMesh())
			{
				StaticMeshes.Add(StaticMeshComponent->GetStaticMesh());
			}
		}
	}

	if (StaticMeshes.Num())
	{
		FFinishCompilationOptions Options;
		Options.bIncludeDependentMeshes = true;
		FinishCompilation(StaticMeshes.Array(), Options);
	}
}

void FStaticMeshCompilingManager::Reschedule()
{
	if (RegisteredStaticMesh.Num() > 1)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FStaticMeshCompilingManager::Reschedule);

		FObjectCacheContextScope ObjectCacheScope;
		TSet<UStaticMesh*> StaticMeshesToProcess;
		for (TWeakObjectPtr<UStaticMesh>& StaticMesh : RegisteredStaticMesh)
		{
			if (StaticMesh.IsValid())
			{
				StaticMeshesToProcess.Add(StaticMesh.Get());
			}
		}

		TMap<UStaticMesh*, float> DistanceToEditingViewport;
		{
			if (StaticMeshesToProcess.Num() > 1)
			{
				const int32 NumViews = IStreamingManager::Get().GetNumViews();
			
				const FStreamingViewInfo* BestViewInfo = nullptr;
				for (int32 ViewIndex = 0; ViewIndex < NumViews; ++ViewIndex)
				{
					const FStreamingViewInfo& ViewInfo = IStreamingManager::Get().GetViewInformation(ViewIndex);
					if (BestViewInfo == nullptr || ViewInfo.BoostFactor > BestViewInfo->BoostFactor)
					{
						BestViewInfo = &ViewInfo;
					}
				}

				const FVector Location = BestViewInfo ? BestViewInfo->ViewOrigin : FVector(0.0f, 0.0f, 0.0f);
				{
					for (UStaticMesh* StaticMesh : StaticMeshesToProcess)
					{
						float NearestStaticMeshDistance = FLT_MAX;
						for (const IStaticMeshComponent* StaticMeshComponent : ObjectCacheScope.GetContext().GetStaticMeshComponents(StaticMesh))
						{
							const IPrimitiveComponent* PrimitiveComponent = StaticMeshComponent->GetPrimitiveComponentInterface();
							if (PrimitiveComponent->IsRegistered())
							{
								FVector ComponentLocation = PrimitiveComponent->GetTransform().GetLocation();
								float ComponentDistance = Location.Dist(ComponentLocation, Location);
								if (ComponentDistance < NearestStaticMeshDistance)
								{
									NearestStaticMeshDistance = ComponentDistance;
								}
							}
						}

						if (NearestStaticMeshDistance != FLT_MAX)
						{
							DistanceToEditingViewport.Add(StaticMesh, NearestStaticMeshDistance);
						}
					}
				}
			}

			if (DistanceToEditingViewport.Num())
			{
				if (FQueuedThreadPoolDynamicWrapper* QueuedThreadPool = (FQueuedThreadPoolDynamicWrapper*)GetThreadPool())
				{
					QueuedThreadPool->Sort(
						[&DistanceToEditingViewport](const IQueuedWork* Lhs, const IQueuedWork* Rhs)
						{
							const FStaticMeshAsyncBuildTask* TaskA = (const FStaticMeshAsyncBuildTask*)Lhs;
							const FStaticMeshAsyncBuildTask* TaskB = (const FStaticMeshAsyncBuildTask*)Rhs;

							const float* ResultA = DistanceToEditingViewport.Find(TaskA->StaticMesh);
							const float* ResultB = DistanceToEditingViewport.Find(TaskB->StaticMesh);

							const float FinalResultA = ResultA ? *ResultA : FLT_MAX;
							const float FinalResultB = ResultB ? *ResultB : FLT_MAX;
							return FinalResultA < FinalResultB;
						}
					);
				}
			}
		}
	}
}

void FStaticMeshCompilingManager::ProcessStaticMeshes(bool bLimitExecutionTime, int32 MinBatchSize)
{
	using namespace StaticMeshCompilingManagerImpl;
	LLM_SCOPE(ELLMTag::StaticMesh);
	TRACE_CPUPROFILER_EVENT_SCOPE(FStaticMeshCompilingManager::ProcessStaticMeshes);
	const int32 NumRemainingMeshes = GetNumRemainingMeshes();
	// Spread out the load over multiple frames but if too many meshes, convergence is more important than frame time
	const int32 MaxMeshUpdatesPerFrame = bLimitExecutionTime ? FMath::Max(64, NumRemainingMeshes / 10) : INT32_MAX;

	FObjectCacheContextScope ObjectCacheScope;
	if (NumRemainingMeshes && NumRemainingMeshes >= MinBatchSize)
	{
		TSet<UStaticMesh*> StaticMeshesToProcess;
		for (TWeakObjectPtr<UStaticMesh>& StaticMesh : RegisteredStaticMesh)
		{
			if (StaticMesh.IsValid())
			{
				StaticMeshesToProcess.Add(StaticMesh.Get());
			}
		}

		{
			TRACE_CPUPROFILER_EVENT_SCOPE(ProcessFinishedStaticMeshes);

			const double TickStartTime = FPlatformTime::Seconds();

			TSet<TWeakObjectPtr<UStaticMesh>> StaticMeshesToPostpone;
			TArray<UStaticMesh*> ProcessedStaticMeshes;
			if (StaticMeshesToProcess.Num())
			{
				for (UStaticMesh* StaticMesh : StaticMeshesToProcess)
				{
					const bool bHasMeshUpdateLeft = ProcessedStaticMeshes.Num() <= MaxMeshUpdatesPerFrame;
					if (bHasMeshUpdateLeft && StaticMesh->IsAsyncTaskComplete())
					{
						PostCompilation(StaticMesh);
						ProcessedStaticMeshes.Add(StaticMesh);
					}
					else
					{
						StaticMeshesToPostpone.Emplace(StaticMesh);
					}
				}
			}

			RegisteredStaticMesh = MoveTemp(StaticMeshesToPostpone);

			if (ProcessedStaticMeshes.Num())
			{
				PostCompilation(ProcessedStaticMeshes);
			}
		}
	}
}

void FStaticMeshCompilingManager::ProcessAsyncTasks(bool bLimitExecutionTime)
{
	FObjectCacheContextScope ObjectCacheScope;
	FinishCompilationsForGame();

	Reschedule();

	ProcessStaticMeshes(bLimitExecutionTime);

	UpdateCompilationNotification();
}

#undef LOCTEXT_NAMESPACE

#endif // #if WITH_EDITOR
