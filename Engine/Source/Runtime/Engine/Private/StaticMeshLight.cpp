// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	StaticMeshLight.cpp: Static mesh lighting code.
=============================================================================*/

#include "StaticMeshLight.h"
#include "Engine/Level.h"
#include "Misc/ConfigCacheIni.h"
#include "RenderUtils.h"
#include "StaticMeshComponentLODInfo.h"
#include "StaticMeshResources.h"
#include "LightMap.h"
#include "Engine/MapBuildDataRegistry.h"
#include "StaticLightingBuildContext.h"
#include "Components/LightComponent.h"
#include "ShadowMap.h"
#include "Engine/StaticMesh.h"
#if WITH_EDITOR
#include "Rendering/StaticLightingSystemInterface.h"
#endif

/**
 * Creates a static lighting vertex to represent the given static mesh vertex.
 * @param VertexBuffer - The static mesh's vertex buffer.
 * @param VertexIndex - The index of the static mesh vertex to access.
 * @param OutVertex - Upon return, contains a static lighting vertex representing the specified static mesh vertex.
 */
static void GetStaticLightingVertex(
	const FPositionVertexBuffer& PositionVertexBuffer,
	const FStaticMeshVertexBuffer& VertexBuffer,
	uint32 VertexIndex,
	const FMatrix& LocalToWorld,
	const FMatrix& LocalToWorldInverseTranspose,
	FStaticLightingVertex& OutVertex
	)
{
	OutVertex.WorldPosition = LocalToWorld.TransformPosition((FVector)PositionVertexBuffer.VertexPosition(VertexIndex));
	OutVertex.WorldTangentX = LocalToWorld.TransformVector((FVector4)VertexBuffer.VertexTangentX(VertexIndex)).GetSafeNormal();
	OutVertex.WorldTangentY = LocalToWorld.TransformVector((FVector)VertexBuffer.VertexTangentY(VertexIndex)).GetSafeNormal();
	OutVertex.WorldTangentZ = LocalToWorldInverseTranspose.TransformVector((FVector4)VertexBuffer.VertexTangentZ(VertexIndex)).GetSafeNormal();

	checkSlow(VertexBuffer.GetNumTexCoords() <= UE_ARRAY_COUNT(OutVertex.TextureCoordinates));
	for(uint32 LightmapTextureCoordinateIndex = 0;LightmapTextureCoordinateIndex < VertexBuffer.GetNumTexCoords();LightmapTextureCoordinateIndex++)
	{
		OutVertex.TextureCoordinates[LightmapTextureCoordinateIndex] = FVector2D(VertexBuffer.GetVertexUV(VertexIndex,LightmapTextureCoordinateIndex));
	}
}


/** Initialization constructor. */
FStaticMeshStaticLightingMesh::FStaticMeshStaticLightingMesh(const UStaticMeshComponent* InPrimitive,int32 InLODIndex,const TArray<ULightComponent*>& InRelevantLights):
	FStaticLightingMesh(
		InPrimitive->GetStaticMesh()->GetRenderData()->LODResources[InLODIndex].GetNumTriangles(),
		InPrimitive->GetStaticMesh()->GetRenderData()->LODResources[InLODIndex].GetNumTriangles(),
		InPrimitive->GetStaticMesh()->GetRenderData()->LODResources[InLODIndex].GetNumVertices(),
		InPrimitive->GetStaticMesh()->GetRenderData()->LODResources[InLODIndex].GetNumVertices(),
		0,
		!!(InPrimitive->CastShadow | InPrimitive->bCastHiddenShadow),
		false,
		InRelevantLights,
		InPrimitive,
		InPrimitive->Bounds.GetBox(),
		InPrimitive->GetStaticMesh()->GetLightingGuid(),
		InPrimitive->LODData[InLODIndex].MapBuildDataId
		),
	LODIndex(InLODIndex),
	StaticMesh(InPrimitive->GetStaticMesh()),
	Primitive(InPrimitive),
	LODRenderData(&InPrimitive->GetStaticMesh()->GetRenderData()->LODResources[InLODIndex]),
	bReverseWinding(InPrimitive->GetComponentTransform().GetDeterminant() < 0.0f)
{
	LODIndexBuffer = LODRenderData->IndexBuffer.GetArrayView();

	// use the primitive's local to world
	SetLocalToWorld(InPrimitive->GetRenderMatrix());
}

FStaticMeshStaticLightingMesh::FStaticMeshStaticLightingMesh() 
	: LODIndex(0)
	, StaticMesh(nullptr)
	, Primitive(nullptr)
	, LODRenderData(nullptr)
	, bReverseWinding(0)
{ 
}

/** 
 * Sets the local to world matrix for this mesh, will also update LocalToWorldInverseTranspose and determinant
 *
 * @param InLocalToWorld Local to world matrix to apply
 */
void FStaticMeshStaticLightingMesh::SetLocalToWorld(const FMatrix& InLocalToWorld)
{
	LocalToWorld = InLocalToWorld;
	LocalToWorldInverseTranspose = LocalToWorld.InverseFast().GetTransposed();
	LocalToWorldDeterminant = LocalToWorld.Determinant();
}



// FStaticLightingMesh interface.

void FStaticMeshStaticLightingMesh::GetTriangle(int32 TriangleIndex,FStaticLightingVertex& OutV0,FStaticLightingVertex& OutV1,FStaticLightingVertex& OutV2) const
{
	// Lookup the triangle's vertex indices.
	const uint32 I0 = LODIndexBuffer[TriangleIndex * 3 + 0];
	const uint32 I1 = LODIndexBuffer[TriangleIndex * 3 + (bReverseWinding ? 2 : 1)];
	const uint32 I2 = LODIndexBuffer[TriangleIndex * 3 + (bReverseWinding ? 1 : 2)];

	// Translate the triangle's static mesh vertices to static lighting vertices.
	check(LODRenderData);
	GetStaticLightingVertex(LODRenderData->VertexBuffers.PositionVertexBuffer,LODRenderData->VertexBuffers.StaticMeshVertexBuffer,I0,LocalToWorld,LocalToWorldInverseTranspose,OutV0);
	GetStaticLightingVertex(LODRenderData->VertexBuffers.PositionVertexBuffer,LODRenderData->VertexBuffers.StaticMeshVertexBuffer,I1,LocalToWorld,LocalToWorldInverseTranspose,OutV1);
	GetStaticLightingVertex(LODRenderData->VertexBuffers.PositionVertexBuffer,LODRenderData->VertexBuffers.StaticMeshVertexBuffer,I2,LocalToWorld,LocalToWorldInverseTranspose,OutV2);
}

void FStaticMeshStaticLightingMesh::GetTriangleIndices(int32 TriangleIndex,int32& OutI0,int32& OutI1,int32& OutI2) const
{
	// Lookup the triangle's vertex indices.
	OutI0 = LODIndexBuffer[TriangleIndex * 3 + 0];
	OutI1 = LODIndexBuffer[TriangleIndex * 3 + (bReverseWinding ? 2 : 1)];
	OutI2 = LODIndexBuffer[TriangleIndex * 3 + (bReverseWinding ? 1 : 2)];
}

bool FStaticMeshStaticLightingMesh::ShouldCastShadow(ULightComponent* Light,const FStaticLightingMapping* Receiver) const
{
	// If the receiver is the same primitive but a different LOD, don't cast shadows on it.
	if(OtherLODs.Contains(Receiver->Mesh))
	{
		return false;
	}
	else
	{
		return FStaticLightingMesh::ShouldCastShadow(Light,Receiver);
	}
}

/** @return		true if the specified triangle casts a shadow. */
bool FStaticMeshStaticLightingMesh::IsTriangleCastingShadow(uint32 TriangleIndex) const
{
	check(LODRenderData);

	// Find the mesh element containing the specified triangle.
	for ( int32 SectionIndex = 0 ; SectionIndex < LODRenderData->Sections.Num() ; ++SectionIndex )
	{
		const FStaticMeshSection& Section = LODRenderData->Sections[ SectionIndex ];
		if ( ( TriangleIndex >= Section.FirstIndex / 3 ) && ( TriangleIndex < Section.FirstIndex / 3 + Section.NumTriangles ) )
		{
			return Section.bCastShadow;
		}
	}

	return true;
}

/** @return		true if the mesh wants to control shadow casting per element rather than per mesh. */
bool FStaticMeshStaticLightingMesh::IsControllingShadowPerElement() const
{
	check(LODRenderData);

	for ( int32 SectionIndex = 0 ; SectionIndex < LODRenderData->Sections.Num() ; ++SectionIndex )
	{
		if ( !LODRenderData->Sections[ SectionIndex ].bCastShadow )
		{
			return true;
		}
	}
	return false;
}

bool FStaticMeshStaticLightingMesh::IsUniformShadowCaster() const
{
	// If this mesh is one of multiple LODs, it won't uniformly shadow all of them.
	return OtherLODs.Num() == 0 && FStaticLightingMesh::IsUniformShadowCaster();
}

const static FName FStaticMeshStaticLightingMesh_IntersectLightRayName("FStaticMeshStaticLightingMesh_IntersectLightRay");

FLightRayIntersection FStaticMeshStaticLightingMesh::IntersectLightRay(const FVector& Start,const FVector& End,bool bFindNearestIntersection) const
{
	// Create the check structure with all the local space fun
	FHitResult Result(1.0f);

	// Do the line check
	FHitResult NewHitInfo;
	FCollisionQueryParams NewTraceParams( SCENE_QUERY_STAT(FStaticMeshStaticLightingMesh_IntersectLightRay), true );
	UStaticMeshComponent* StaticMeshComp = const_cast<UStaticMeshComponent*>(Primitive);
	const bool bIntersects = StaticMeshComp->LineTraceComponent( Result, Start, End, NewTraceParams );
	
	// Setup a vertex to represent the intersection.
	FStaticLightingVertex IntersectionVertex;
	if(bIntersects)
	{
		IntersectionVertex.WorldPosition = Result.Location;
		IntersectionVertex.WorldTangentZ = Result.Normal;
	}
	else
	{
		IntersectionVertex.WorldPosition = FVector(0,0,0);
		IntersectionVertex.WorldTangentZ = FVector(0,0,1);
	}
	return FLightRayIntersection(bIntersects,IntersectionVertex);
}

/** Initialization constructor. */
FStaticMeshStaticLightingTextureMapping::FStaticMeshStaticLightingTextureMapping(
	UStaticMeshComponent* InPrimitive,
	int32 InLODIndex,
	FStaticLightingMesh* InMesh,
	int32 InSizeX,
	int32 InSizeY,
	int32 InLightmapTextureCoordinateIndex,
	bool bPerformFullQualityRebuild
	):
	FStaticLightingTextureMapping(
		InMesh,
		InPrimitive,
		InSizeX,
		InSizeY,
		InLightmapTextureCoordinateIndex
		),
	Primitive(InPrimitive),
	LODIndex(InLODIndex)
{}

#if WITH_EDITOR
void FStaticMeshStaticLightingTextureMapping::Serialize(FArchive& Ar)
{
	FStaticLightingTextureMapping::Serialize(Ar);

	FSoftObjectPath PrimitivePath(Primitive.Get());
	Ar << PrimitivePath;
	Primitive = TWeakObjectPtr<UStaticMeshComponent>(Cast<UStaticMeshComponent>(PrimitivePath.ResolveObject()));
	
	Ar << LODIndex;
}

// FStaticLightingTextureMapping interface
void FStaticMeshStaticLightingTextureMapping::Apply(FQuantizedLightmapData* QuantizedData, const TMap<ULightComponent*,FShadowMapData2D*>& ShadowMapData, const FStaticLightingBuildContext* LightingContext)
{
	const bool bUseVirtualTextures = UseVirtualTextureLightmap(GMaxRHIShaderPlatform);

	UStaticMeshComponent* StaticMeshComponent = Primitive.Get();

	if (StaticMeshComponent && StaticMeshComponent->GetOwner() && StaticMeshComponent->GetOwner()->GetLevel())
	{
		// Should have happened at a higher level
		check(!StaticMeshComponent->IsRenderStateCreated());
		// The rendering thread reads from LODData and IrrelevantLights, therefore
		// the component must have finished detaching from the scene on the rendering
		// thread before it is safe to continue.
		check(StaticMeshComponent->GetSceneData().AttachmentCounter.GetValue() == 0);

		// Ensure LODData has enough entries in it, free not required.
		const bool bLODDataCountChanged = StaticMeshComponent->SetLODDataCount(LODIndex + 1, StaticMeshComponent->GetStaticMesh()->GetNumLODs());
		if (bLODDataCountChanged)
		{
			StaticMeshComponent->MarkPackageDirty();
		}

		FStaticMeshComponentLODInfo& ComponentLODInfo = StaticMeshComponent->LODData[LODIndex];

		// Ensure this LODInfo has a valid MapBuildDataId
		if (ComponentLODInfo.CreateMapBuildDataId(LODIndex))
		{
			StaticMeshComponent->MarkPackageDirty();
		}

		ELightMapPaddingType PaddingType = GAllowLightmapPadding ? LMPT_NormalPadding : LMPT_NoPadding;
		const bool bHasNonZeroData = (QuantizedData != NULL && QuantizedData->HasNonZeroData());

		UMapBuildDataRegistry* Registry = LightingContext->GetOrCreateRegistryForActor(StaticMeshComponent->GetOwner());
		FMeshMapBuildData& MeshBuildData = Registry->AllocateMeshBuildData(ComponentLODInfo.MapBuildDataId, true);

		// We always create a light map if the surface either has any non-zero lighting data, or if the surface has a shadow map.  The runtime
		// shaders are always expecting a light map in the case of a shadow map, even if the lighting is entirely zero.  This is simply to reduce
		// the number of shader permutations to support in the very unlikely case of a unshadowed surfaces that has lighting values of zero.
		const bool bNeedsLightMap = bHasNonZeroData || ShadowMapData.Num() > 0 || Mesh->RelevantLightsGuid.Num() > 0 || (QuantizedData != NULL && QuantizedData->bHasSkyShadowing);
		if (bNeedsLightMap)
		{
			// Create a light-map for the primitive.
			TMap<ULightComponent*, FShadowMapData2D*> EmptyShadowMapData;
			MeshBuildData.LightMap = FLightMap2D::AllocateLightMap(
				Registry,
				QuantizedData,
				bUseVirtualTextures ? ShadowMapData : EmptyShadowMapData,
				StaticMeshComponent->Bounds,
				PaddingType,
				LMF_Streamed
			);
		}
		else
		{
			MeshBuildData.LightMap = NULL;
		}

		if (ShadowMapData.Num() > 0 && !bUseVirtualTextures)
		{
			MeshBuildData.ShadowMap = FShadowMap2D::AllocateShadowMap(
				Registry,
				ShadowMapData,
				StaticMeshComponent->Bounds,
				PaddingType,
				SMF_Streamed
				);
		}
		else
		{
			MeshBuildData.ShadowMap = NULL;
		}

		// Build the list of statically irrelevant lights.
		// IrrelevantLights was cleared in InvalidateLightingCacheDetailed

		for(int32 LightIndex = 0;LightIndex < Mesh->RelevantLightsGuid.Num();LightIndex++)
		{
			FGuid LightGuid = Mesh->RelevantLightsGuid[LightIndex];

			// Check if the light is stored in the light-map.
			const bool bIsInLightMap = MeshBuildData.LightMap && MeshBuildData.LightMap->LightGuids.Contains(LightGuid);

			// Check if the light is stored in the shadow-map.
			const bool bIsInShadowMap = MeshBuildData.ShadowMap && MeshBuildData.ShadowMap->LightGuids.Contains(LightGuid);

			// Add the light to the statically irrelevant light list if it is in the potentially relevant light list, but didn't contribute to the light-map.
			if(!bIsInLightMap && !bIsInShadowMap)
			{	
				MeshBuildData.IrrelevantLights.AddUnique(LightGuid);
			}
		}
	}
}

void UStaticMeshComponent::GetStaticLightingInfo(FStaticLightingPrimitiveInfo& OutPrimitiveInfo,const TArray<ULightComponent*>& InRelevantLights,const FLightingBuildOptions& Options)
{
	if( HasValidSettingsForStaticLighting(false) )
	{
		int32		BaseLightMapWidth	= 0;
		int32		BaseLightMapHeight	= 0;
		GetLightMapResolution( BaseLightMapWidth, BaseLightMapHeight );

		TArray<FStaticMeshStaticLightingMesh*> StaticLightingMeshes;
		bool bCanLODsShareStaticLighting = GetStaticMesh()->CanLODsShareStaticLighting();
		int32 NumLODs = bCanLODsShareStaticLighting ? 1 : GetStaticMesh()->GetRenderData()->LODResources.Num();
		for(int32 LODIndex = 0;LODIndex < NumLODs;LODIndex++)
		{
			const FStaticMeshLODResources& LODRenderData = GetStaticMesh()->GetRenderData()->LODResources[LODIndex];
			// Figure out whether we are storing the lighting/ shadowing information in a texture or vertex buffer.
			bool bValidTextureMap;
			if( (BaseLightMapWidth > 0) && (BaseLightMapHeight > 0) 
				&& GetStaticMesh()->GetLightMapCoordinateIndex() >= 0
				&& (uint32)GetStaticMesh()->GetLightMapCoordinateIndex() < LODRenderData.VertexBuffers.StaticMeshVertexBuffer.GetNumTexCoords())
			{
				bValidTextureMap = true;
			}
			else
			{
				bValidTextureMap = false;
			}

			if (bValidTextureMap)
			{
				// We need to make sure this data is created before trying to create the FStaticMeshStaticLightingMesh
				UpdateStaticLightingData();
				UpdateMapBuildDataId();

				// Create a static lighting mesh for the LOD.
				FStaticMeshStaticLightingMesh* StaticLightingMesh = AllocateStaticLightingMesh(LODIndex,InRelevantLights);
				OutPrimitiveInfo.Meshes.Add(StaticLightingMesh);
				StaticLightingMeshes.Add(StaticLightingMesh);

				// Shrink LOD texture lightmaps by half for each LOD level
				const int32 LightMapWidth = LODIndex > 0 ? FMath::Max(BaseLightMapWidth / (2 << (LODIndex - 1)), 32) : BaseLightMapWidth;
				const int32 LightMapHeight = LODIndex > 0 ? FMath::Max(BaseLightMapHeight / (2 << (LODIndex - 1)), 32) : BaseLightMapHeight;

				if (GetLightmapType() == ELightmapType::ForceVolumetric)
				{
					OutPrimitiveInfo.Mappings.Add(new FStaticLightingGlobalVolumeMapping(
						StaticLightingMesh,this,LightMapWidth,LightMapHeight, GetStaticMesh()->GetLightMapCoordinateIndex()));
				}
				else
				{
					// Create a static lighting texture mapping for the LOD.
					OutPrimitiveInfo.Mappings.Add(new FStaticMeshStaticLightingTextureMapping(
						this,LODIndex,StaticLightingMesh,LightMapWidth,LightMapHeight, GetStaticMesh()->GetLightMapCoordinateIndex(),true));
				}
			}
		}

		// Give each LOD's static lighting mesh a list of the other LODs of this primitive, so they can disallow shadow casting between LODs.
		for(int32 MeshIndex = 0;MeshIndex < StaticLightingMeshes.Num();MeshIndex++)
		{
			for(int32 OtherMeshIndex = 0;OtherMeshIndex < StaticLightingMeshes.Num();OtherMeshIndex++)
			{
				if(MeshIndex != OtherMeshIndex)
				{
					StaticLightingMeshes[MeshIndex]->OtherLODs.Add(StaticLightingMeshes[OtherMeshIndex]);
				}
			}
		}
	}
}

void UStaticMeshComponent::AddMapBuildDataGUIDs(TSet<FGuid>& InGUIDs) const
{
	for (const FStaticMeshComponentLODInfo& ComponentLODInfo : LODData)
	{
		InGUIDs.Add(ComponentLODInfo.MapBuildDataId);
	}
}
#endif


ELightMapInteractionType UStaticMeshComponent::GetStaticLightingType() const
{
	ELightMapInteractionType InteractionType = LMIT_None;

	bool bUseTextureMap = false;
	if( HasValidSettingsForStaticLighting(false) )
	{
		if (GetLightmapType() == ELightmapType::ForceVolumetric)
		{
			InteractionType = LMIT_GlobalVolume;
		}
		else
		{
			// Process each LOD separately.
			for(int32 LODIndex = 0;LODIndex < GetStaticMesh()->GetRenderData()->LODResources.Num();LODIndex++)
			{
				const FStaticMeshLODResources& LODRenderData = GetStaticMesh()->GetRenderData()->LODResources[LODIndex];

				// Figure out whether we are storing the lighting/ shadowing information in a texture or vertex buffer.
				int32		LightMapWidth	= 0;
				int32		LightMapHeight	= 0;
				GetLightMapResolution( LightMapWidth, LightMapHeight );

				if ((LightMapWidth > 0) && (LightMapHeight > 0) &&	
					(GetStaticMesh()->GetLightMapCoordinateIndex() >= 0) &&
					((uint32)GetStaticMesh()->GetLightMapCoordinateIndex() < LODRenderData.VertexBuffers.StaticMeshVertexBuffer.GetNumTexCoords())
					)
				{
					InteractionType = LMIT_Texture;
					break;
				}
			}
		}
	}

	return InteractionType;
}

bool UStaticMeshComponent::IsPrecomputedLightingValid() const
{
	if (GetLightmapType() == ELightmapType::ForceVolumetric)
	{
		// No unbuilt tracking mechanism
		return true;
	}

	if (LODData.Num() > 0)
	{
		return GetMeshMapBuildData(LODData[0]) != NULL;
	}

#if WITH_EDITOR
	return FStaticLightingSystemInterface::GetPrimitiveMeshMapBuildData(this, MinLOD) != nullptr;
#else
	return false;
#endif
}

float UStaticMeshComponent::GetEmissiveBoost(int32 ElementIndex) const
{
	return LightmassSettings.EmissiveBoost;
}

float UStaticMeshComponent::GetDiffuseBoost(int32 ElementIndex) const
{
	return LightmassSettings.DiffuseBoost;
}

FStaticMeshStaticLightingMesh* UStaticMeshComponent::AllocateStaticLightingMesh(int32 LODIndex, const TArray<ULightComponent*>& InRelevantLights)
{
	return new FStaticMeshStaticLightingMesh(this,LODIndex,InRelevantLights);
}

void UStaticMeshComponent::InvalidateLightingCacheDetailed(bool bInvalidateBuildEnqueuedLighting, bool bTranslationOnly)
{
#if WITH_EDITOR
	// If still compiling, static lighting is not registered and doesn't need unregistration
	if (!IsCompiling() && HasStaticLighting() && HasValidSettingsForStaticLighting(false))
	{
		FStaticLightingSystemInterface::OnPrimitiveComponentUnregistered.Broadcast(this);
	}
#endif

	// Save the static mesh state for transactions, force it to be marked dirty if we are going to discard any static lighting data.
	Modify(true);

	Super::InvalidateLightingCacheDetailed(bInvalidateBuildEnqueuedLighting, bTranslationOnly);

	for(int32 i = 0; i < LODData.Num(); i++)
	{
		FStaticMeshComponentLODInfo& LODDataElement = LODData[i];
		LODDataElement.OriginalMapBuildDataId.Invalidate();
		LODDataElement.MapBuildDataId.Invalidate();
	}

#if WITH_EDITOR
	// If still compiling, static lighting will be registered when compilation finishes
	if (!IsCompiling() && HasStaticLighting() && HasValidSettingsForStaticLighting(false))
	{
		FStaticLightingSystemInterface::OnPrimitiveComponentRegistered.Broadcast(this);
	}
#endif

	MarkRenderStateDirty();
}

UObject const* UStaticMeshComponent::AdditionalStatObject() const
{
	return GetStaticMesh();
}

bool UStaticMeshComponent::SetStaticLightingMapping(bool bTextureMapping, int32 ResolutionToUse)
{
	bool bSuccessful = false;
	if (GetStaticMesh())
	{
		if (bTextureMapping == true)
		{
			// Set it to texture mapping!
			if (ResolutionToUse == 0)
			{
				if (bOverrideLightMapRes == true)
				{
					// If overriding the static mesh setting, check to set if set to 0
					// which will force the component to use vertex mapping
					if (OverriddenLightMapRes == 0)
					{
						// See if the static mesh has a valid setting
						if (GetStaticMesh()->GetLightMapResolution() != 0)
						{
							// Simply uncheck the override...
							bOverrideLightMapRes = false;
							bSuccessful = true;
						}
						else
						{
							// Set it to the default value from the ini
							int32 TempInt = 0;
							verify(GConfig->GetInt(TEXT("DevOptions.StaticLighting"), TEXT("DefaultStaticMeshLightingRes"), TempInt, GLightmassIni));
							OverriddenLightMapRes = TempInt;
							bSuccessful = true;
						}
					}
					else
					{
						// We should be texture mapped already...
					}
				}
				else
				{
					// See if the static mesh has a valid setting
					if (GetStaticMesh()->GetLightMapResolution() == 0)
					{
						// See if the static mesh has a valid setting
						if (OverriddenLightMapRes != 0)
						{
							// Simply check the override...
							bOverrideLightMapRes = true;
							bSuccessful = true;
						}
						else
						{
							// Set it to the default value from the ini
							int32 TempInt = 0;
							verify(GConfig->GetInt(TEXT("DevOptions.StaticLighting"), TEXT("DefaultStaticMeshLightingRes"), TempInt, GLightmassIni));
							OverriddenLightMapRes = TempInt;
							bOverrideLightMapRes = true;
							bSuccessful = true;
						}
					}
					else
					{
						// We should be texture mapped already...
					}
				}
			}
			else
			{
				// Use the override - even if it was already set to override at a different value
				OverriddenLightMapRes = ResolutionToUse;
				bOverrideLightMapRes = true;
				bSuccessful = true;
			}
		}
		else
		{
			// Set it to vertex mapping...
			if (bOverrideLightMapRes == true)
			{
				if (OverriddenLightMapRes != 0)
				{
					// See if the static mesh has a valid setting
					if (GetStaticMesh()->GetLightMapResolution() == 0)
					{
						// Simply uncheck the override...
						bOverrideLightMapRes = false;
						bSuccessful = true;
					}
					else
					{
						// Set it to 0 to force vertex mapping
						OverriddenLightMapRes = 0;
						bSuccessful = true;
					}
				}
				else
				{
					// We should be vertex mapped already...
				}
			}
			else
			{
				// See if the static mesh has a valid setting
				if (GetStaticMesh()->GetLightMapResolution() != 0)
				{
					// Set it to the default value from the ini
					OverriddenLightMapRes = 0;
					bOverrideLightMapRes = true;
					bSuccessful = true;
				}
				else
				{
					// We should be vertex mapped already...
				}
			}
		}
	}

	if (bSuccessful == true)
	{
		MarkPackageDirty();
	}

	return bSuccessful;
}
