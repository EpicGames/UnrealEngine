// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	StaticMeshResources.cpp
=============================================================================*/

#include "StaticMeshResources.h"
#include "RenderingThread.h"
#include "SceneInterface.h"
#include "UObject/UObjectIterator.h"
#include "Engine/SimpleConstructionScript.h"

#include "ObjectCacheContext.h"

FStaticMeshComponentRecreateRenderStateContext::FStaticMeshComponentRecreateRenderStateContext(UStaticMesh* InStaticMesh, bool InUnbuildLighting, bool InRefreshBounds)
	: FStaticMeshComponentRecreateRenderStateContext(TArray<UStaticMesh*>{ InStaticMesh }, InUnbuildLighting, InRefreshBounds)
{
}

FStaticMeshComponentRecreateRenderStateContext::FStaticMeshComponentRecreateRenderStateContext(const TArray<UStaticMesh*>& InStaticMeshes, bool InUnbuildLighting, bool InRefreshBounds)
	: bUnbuildLighting(InUnbuildLighting)
	, bRefreshBounds(InRefreshBounds)
{
	StaticMeshComponents.Reserve(InStaticMeshes.Num());
	for (UStaticMesh* StaticMesh : InStaticMeshes)
	{
		if (StaticMesh)
		{
			StaticMeshComponents.Add(StaticMesh);
		}
	}

	if (StaticMeshComponents.Num())
	{
		TSet<FSceneInterface*> Scenes;

		FObjectCacheContextScope ObjectCacheScope;
		for (auto It = StaticMeshComponents.CreateIterator(); It; ++It)
		{
			for (IStaticMeshComponent* Component : ObjectCacheScope.GetContext().GetStaticMeshComponents((UStaticMesh*)It.Key()))
			{
				IPrimitiveComponent* PrimitiveComponent = Component->GetPrimitiveComponentInterface();

				if (PrimitiveComponent->IsRenderStateCreated())
				{
					check(PrimitiveComponent->IsRegistered());
					PrimitiveComponent->DestroyRenderState();

					if (UStaticMeshComponent* StaticMeshComponent = Cast<UStaticMeshComponent>(PrimitiveComponent->GetUObject()))
					{
						It.Value().UStaticMeshComponents.Add(StaticMeshComponent);
					}
					It.Value().IStaticMeshComponents.Add(Component);
					Scenes.Add(PrimitiveComponent->GetScene());
				}
			}
		}

		if (Scenes.Num())
		{
			UpdateAllPrimitiveSceneInfosForScenes(MoveTemp(Scenes));

			// Flush the rendering commands generated by the detachments.
			// The static mesh scene proxies reference the UStaticMesh, and this ensures that they are cleaned up before the UStaticMesh changes.
			FlushRenderingCommands();
		}
	}
}

const TArray<UStaticMeshComponent*>& FStaticMeshComponentRecreateRenderStateContext::GetComponentsUsingMesh(UStaticMesh* StaticMesh) const
{
	return StaticMeshComponents.FindChecked(StaticMesh).UStaticMeshComponents;
}

FStaticMeshComponentRecreateRenderStateContext::~FStaticMeshComponentRecreateRenderStateContext()
{
	if (StaticMeshComponents.Num())
	{
		TSet<FSceneInterface*> Scenes;

		for (const auto& MeshComponents : StaticMeshComponents)
		{
			for (UStaticMeshComponent* Component : MeshComponents.Value.UStaticMeshComponents)
			{
				if (bUnbuildLighting)
				{
					// Invalidate the component's static lighting.
					// This unregisters and reregisters so must not be in the constructor
					Component->InvalidateLightingCache();
				}

				if (bRefreshBounds)
				{
					Component->UpdateBounds();
				}

			}

			for (IStaticMeshComponent* Component : MeshComponents.Value.IStaticMeshComponents)
			{
				IPrimitiveComponent* PrimitiveComponent = Component->GetPrimitiveComponentInterface();

				if (PrimitiveComponent->IsRegistered() && !PrimitiveComponent->IsRenderStateCreated())
				{
					PrimitiveComponent->CreateRenderState(nullptr);
					Scenes.Add(PrimitiveComponent->GetScene());
				}
			}
		}

		UpdateAllPrimitiveSceneInfosForScenes(MoveTemp(Scenes));
	}
}

FStaticMeshComponentBulkReregisterContext::FStaticMeshComponentBulkReregisterContext(
	FSceneInterface* InScene, TArrayView<UActorComponent*> InComponents, EBulkReregister ReregisterType)
{
	Scene = InScene;
	StaticMeshComponents.Reserve(InComponents.Num());

	TArray<UPrimitiveComponent*> ComponentsWithRenderState;
	ComponentsWithRenderState.Reserve(InComponents.Num());

	// In some rare cases, the caller may pass multiple pointers to the same component.  In that case, we must only reregister it once.
	// The duplicate tracker is only updated when we have a possible duplicate (detected by checking if bBulkReregister is already set),
	// to avoid overhead in in the common case where there are no duplicates.
	TSet<UActorComponent*> DuplicateTracker;

	for (int32 ComponentIndex = 0; ComponentIndex < InComponents.Num(); ++ComponentIndex)
	{
		UActorComponent* Component = InComponents[ComponentIndex];
		UStaticMeshComponent* StaticMeshComponent = Cast<UStaticMeshComponent>(Component);

		// When running a render state update, only batch process components that are dirty and not hidden
		if (StaticMeshComponent && StaticMeshComponent->SceneProxy &&
			(ReregisterType == EBulkReregister::Component || (StaticMeshComponent->IsRenderStateDirty() && StaticMeshComponent->ShouldComponentAddToScene())))
		{
			// Do we have a potential duplicate?
			if (StaticMeshComponent->bBulkReregister)
			{
				// Initialize the duplicate tracker with the elements before the current one.
				if (DuplicateTracker.IsEmpty())
				{
					DuplicateTracker.Reserve(InComponents.Num());
					for (int32 PreviousIndex = 0; PreviousIndex < ComponentIndex; ++PreviousIndex)
					{
						DuplicateTracker.Add(InComponents[PreviousIndex]);
					}
				}
			}

			// Check if the current item is a duplicate.
			if (!DuplicateTracker.IsEmpty())
			{
				if (DuplicateTracker.Find(Component))
				{
					continue;
				}
				DuplicateTracker.Add(Component);
			}

			StaticMeshComponents.Add(StaticMeshComponent);

			// If bBulkReregister is set, and it's NOT a duplicate, it implies a code bug (for example, nesting of a reregister context, which is not supported).
			check(StaticMeshComponent->bBulkReregister == false);
			StaticMeshComponent->bBulkReregister = true;

			if (StaticMeshComponent->IsRenderStateCreated())
			{
				ComponentsWithRenderState.Add(StaticMeshComponent);
			}
		}
	}

	if (ComponentsWithRenderState.Num())
	{
		// NOTE:  We deliberately don't bother clearing the physics render debug data via SendRenderDebugPhysics
		// (or a batched equivalent), as removing the primitive deletes the scene proxy, including the debug data.
		Scene->BatchRemovePrimitives(ComponentsWithRenderState);

		// When running a render state update, we don't need to run "BatchReleasePrimitives", since the component isn't
		// being regenerated, just the render state (see DoDeferredRenderUpdates_Concurrent in LevelTick.cpp).
		if (ReregisterType == EBulkReregister::Component)
		{
			Scene->BatchReleasePrimitives(ComponentsWithRenderState);
		}
	}
}

FStaticMeshComponentBulkReregisterContext::~FStaticMeshComponentBulkReregisterContext()
{
	// Remove any components that are pending kill, in case a creation script rebuilt the component from scratch (versus just re-registering).
	StaticMeshComponents.RemoveAllSwap([](const UActorComponent* Component) { return !IsValidChecked(Component); }, EAllowShrinking::No);

	if (StaticMeshComponents.Num())
	{
		Scene->BatchAddPrimitives(StaticMeshComponents);
#if UE_ENABLE_DEBUG_DRAWING
		UPrimitiveComponent::BatchSendRenderDebugPhysics(StaticMeshComponents);
#endif
	}

	for (UPrimitiveComponent* StaticMeshComponent : StaticMeshComponents)
	{
		check(StaticMeshComponent->bBulkReregister == true);
		StaticMeshComponent->bBulkReregister = false;
	}

	for (USimpleConstructionScript* SCS : SCSs)
	{
		SCS->ReregisterContext = nullptr;
	}
}

void FStaticMeshComponentBulkReregisterContext::AddSimpleConstructionScript(USimpleConstructionScript* SCS)
{
	check(SCS);

	// Check if we already added this SCS to this reregister context
	if (SCS->ReregisterContext != this)
	{
		check(SCS->ReregisterContext == nullptr);
		SCS->ReregisterContext = this;
		SCSs.Add(SCS);
	}
}

void FStaticMeshComponentBulkReregisterContext::SanitizeMeshComponents()
{
	// for contexts in which side effects are less predictable, e.g. in editor:
	// Note: we must clear bBulkReregister on any component we stop tracking here.  The context destructor only clears the flag
	// on components that remain in StaticMeshComponents, so a component dropped by this filter would leak the flag, triggering
	// downstream validation asserts.
	StaticMeshComponents.RemoveAllSwap([](UPrimitiveComponent* Component)
		{
			if (!IsValidChecked(Component) || !Component->IsRegistered() || Component->SceneProxy != nullptr)
			{
				// Only touch the flag on objects that are still valid; pending-kill objects are being torn down anyway.
				if (IsValid(Component))
				{
					Component->bBulkReregister = false;
				}
				return true;
			}
			return false;
		}, EAllowShrinking::No);
}

void FStaticMeshComponentBulkReregisterContext::AddConstructedComponent(USceneComponent* SceneComp)
{
	UStaticMeshComponent* StaticMeshComponent = Cast<UStaticMeshComponent>(SceneComp);
	if (StaticMeshComponent)
	{
		check(StaticMeshComponent->bBulkReregister == false);
		StaticMeshComponent->bBulkReregister = true;

		StaticMeshComponents.Add(StaticMeshComponent);
	}
}
