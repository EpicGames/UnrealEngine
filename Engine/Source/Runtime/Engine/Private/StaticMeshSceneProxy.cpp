// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	StaticMeshSceneProxy.cpp: Static mesh rendering code.
=============================================================================*/

#include "StaticMeshSceneProxy.h"
#include "BodySetupEnums.h"
#include "EngineStats.h"
#include "EngineLogs.h"
#include "PrimitiveViewRelevance.h"
#include "LightMap.h"
#include "MaterialDomain.h"
#include "RenderUtils.h"
#include "SceneInterface.h"
#include "Engine/MapBuildDataRegistry.h"
#include "Materials/Material.h"
#include "Materials/MaterialRenderProxy.h"
#include "MaterialShared.h"
#include "Engine/MaterialOverlayHelper.h"
#include "Engine/StaticMesh.h"
#include "ComponentReregisterContext.h"
#include "EngineUtils.h"
#include "SceneView.h"
#include "StaticMeshComponentLODInfo.h"
#include "StaticMeshResources.h"
#include "PhysicalMaterials/PhysicalMaterialMask.h"

#include "DistanceFieldAtlas.h"
#include "MeshCardRepresentation.h"
#include "Components/BrushComponent.h"
#include "AI/Navigation/NavCollisionBase.h"
#include "ComponentRecreateRenderStateContext.h"
#include "PhysicsEngine/BodySetup.h"
#include "Engine/LODActor.h"
#include "WorldPartition/HLOD/HLODActor.h"
#include "UnrealEngine.h"
#include "PrimitiveSceneInfo.h"
#include "NaniteSceneProxy.h"
#include "RenderCore.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "EngineModule.h"
#include "MeshPaintVisualize.h"
#include "ShadowMap.h"
#include "VT/MeshPaintVirtualTexture.h"
#include "TextureResource.h"

#include "StaticMeshSceneProxyDesc.h"
#include "StaticMeshComponentHelper.h"
#include "MaterialCache/MaterialCache.h"
#include "MaterialCache/MaterialCacheVirtualTexture.h"
#include "Rendering/NaniteResourcesHelper.h"
#include "PSOPrecacheSettings.h"

#if WITH_EDITOR
#include "Rendering/StaticLightingSystemInterface.h"
#endif

/** If true, optimized depth-only index buffers are used for shadow rendering. */
static bool GUseShadowIndexBuffer = true;

/** If true, reversed index buffer are used for mesh with negative transform determinants. */
static bool GUseReversedIndexBuffer = true;

static void ToggleShadowIndexBuffers()
{
	FlushRenderingCommands();
	GUseShadowIndexBuffer = !GUseShadowIndexBuffer;
	UE_LOGF(LogStaticMesh,Log,"Optimized shadow index buffers %ls",GUseShadowIndexBuffer ? TEXT("ENABLED") : TEXT("DISABLED"));
	FGlobalComponentReregisterContext ReregisterContext;
}

static void ToggleReversedIndexBuffers()
{
	FlushRenderingCommands();
	GUseReversedIndexBuffer = !GUseReversedIndexBuffer;
	UE_LOGF(LogStaticMesh,Log,"Reversed index buffers %ls",GUseReversedIndexBuffer ? TEXT("ENABLED") : TEXT("DISABLED"));
	FGlobalComponentReregisterContext ReregisterContext;
}

static FAutoConsoleCommand GToggleShadowIndexBuffersCmd(
	TEXT("ToggleShadowIndexBuffers"),
	TEXT("Render static meshes with an optimized shadow index buffer that minimizes unique vertices."),
	FConsoleCommandDelegate::CreateStatic(ToggleShadowIndexBuffers)
	);

static bool GStaticMeshComponentBoostPSOPrecachePri = false;
static FAutoConsoleVariableRef CVarStaticMeshComponentBoostPSOPrecachePri(
	TEXT("r.PSOPrecache.StaticMeshComponentPSOPrecachePriority"),
	GStaticMeshComponentBoostPSOPrecachePri,
	TEXT("Static Mesh component PSO precache priority level.\n")
	TEXT(" 0. Static Mesh component's PSO precache requests are set to high priority (default)\n")
	TEXT(" 1. Static Mesh component's PSO precache requests are set to highest priority"),
	ECVF_Default
);

EPSOPrecachePriority GetStaticMeshComponentBoostPSOPrecachePriority()
{
	return GStaticMeshComponentBoostPSOPrecachePri ? EPSOPrecachePriority::Highest : EPSOPrecachePriority::High;
}

static FAutoConsoleCommand GToggleReversedIndexBuffersCmd(
	TEXT("ToggleReversedIndexBuffers"),
	TEXT("Render static meshes with negative transform determinants using a reversed index buffer."),
	FConsoleCommandDelegate::CreateStatic(ToggleReversedIndexBuffers)
	);

// TODO: Should move this outside of SM, since Nanite can be used for multiple primitive types
TAutoConsoleVariable<int32> CVarRenderNaniteMeshes(
	TEXT("r.Nanite"),
	1,
	TEXT("Render static meshes using Nanite."),
	FConsoleVariableDelegate::CreateLambda([](IConsoleVariable* InVariable)
	{
		FGlobalComponentRecreateRenderStateContext Context;
	}),
	ECVF_Scalability | ECVF_RenderThreadSafe
);

// TODO: Should move this outside of SM, since Nanite can be used for multiple primitive types
namespace Nanite
{
	int32 GEnableNaniteMaterialOverrides = 1;
	FAutoConsoleVariableRef CVarEnableNaniteNaterialOverrides(
		TEXT("r.Nanite.MaterialOverrides"),
		GEnableNaniteMaterialOverrides,
		TEXT("Enable support for Nanite specific material overrides."),
		FConsoleVariableDelegate::CreateLambda([](IConsoleVariable* InVariable)
		{
			FGlobalComponentRecreateRenderStateContext Context;
		}),
		ECVF_Scalability | ECVF_RenderThreadSafe
	);
}

bool GForceDefaultMaterial = false;

static void ToggleForceDefaultMaterial()
{
	FlushRenderingCommands();
	GForceDefaultMaterial = !GForceDefaultMaterial;
	UE_LOGF(LogStaticMesh,Log,"Force default material %ls",GForceDefaultMaterial ? TEXT("ENABLED") : TEXT("DISABLED"));
	FGlobalComponentReregisterContext ReregisterContext;
}

static FAutoConsoleCommand GToggleForceDefaultMaterialCmd(
	TEXT("ToggleForceDefaultMaterial"),
	TEXT("Render all meshes with the default material."),
	FConsoleCommandDelegate::CreateStatic(ToggleForceDefaultMaterial)
	);

static TAutoConsoleVariable<int32> CVarRayTracingStaticMeshes(
	TEXT("r.RayTracing.Geometry.StaticMeshes"),
	1,
	TEXT("Defines if static meshes are added to the ray tracing scene.\n")
	TEXT(" 0: raytracing cached MDCs for static mesh are not created.\n")
	TEXT(" 1: raytracing cached MDCs for static mesh are created and instances are added to the RayTracing scene (default).\n")
	TEXT(" 2: static meshes are runtime culled from the RayTracing scene, but keeps the cached state for fast toggling."));

static TAutoConsoleVariable<int32> CVarRayTracingStaticMeshesWPO(
	TEXT("r.RayTracing.Geometry.StaticMeshes.WPO"),
	1,
	TEXT("World position offset evaluation for static meshes with EvaluateWPO enabled in ray tracing effects.\n")
	TEXT(" 0: static meshes with world position offset hidden in ray tracing.\n")
	TEXT(" 1: static meshes with world position offset visible in ray tracing, WPO evaluation enabled (default).\n")
	TEXT(" 2: static meshes with world position offset visible in ray tracing, WPO evaluation disabled.")
);

FAutoConsoleVariableSink CVarRayTracingStaticMeshesWPOSink(FConsoleCommandDelegate::CreateLambda([]()
{
	static int32 CachedRayTracingStaticMeshesWPO = CVarRayTracingStaticMeshesWPO.GetValueOnGameThread();

	int32 RayTracingStaticMeshesWPO = CVarRayTracingStaticMeshesWPO.GetValueOnGameThread();

	if (CachedRayTracingStaticMeshesWPO != RayTracingStaticMeshesWPO)
	{
		CachedRayTracingStaticMeshesWPO = RayTracingStaticMeshesWPO;

		// NV-JIRA UE-668: Do this as a task on the game thread to break up a possible
		// reentry call to USkeletalMeshComponent::PostAnimEvaluation if BP toggles this CVar
		FFunctionGraphTask::CreateAndDispatchWhenReady([]()
		{
			// Easiest way to update all static scene proxies is to recreate them
			FlushRenderingCommands();
			FGlobalComponentReregisterContext ReregisterContext;
			
		}, TStatId(), nullptr, ENamedThreads::GameThread);
	}
}));

static TAutoConsoleVariable<int32> CVarRayTracingStaticMeshesWPOCulling(
	TEXT("r.RayTracing.Geometry.StaticMeshes.WPO.Culling"),
	1,
	TEXT("Enable culling for WPO evaluation for static meshes in ray tracing (default = 1 (Culling enabled))"));

static TAutoConsoleVariable<float> CVarRayTracingStaticMeshesWPOCullingRadius(
	TEXT("r.RayTracing.Geometry.StaticMeshes.WPO.CullingRadius"),
	12000.0f, // 120 m
	TEXT("Do not evaluate world position offset for static meshes outside of this radius in ray tracing effects (default = 12000 (120m))"));


void GetStaticMeshRayTracingWPOConfig(bool& bOutHasRayTracingRepresentation, bool& bOutDynamicRayTracingGeometry)
{
	bOutHasRayTracingRepresentation = CVarRayTracingStaticMeshesWPO.GetValueOnAnyThread() != 0;
	bOutDynamicRayTracingGeometry = CVarRayTracingStaticMeshesWPO.GetValueOnAnyThread() == 1;
}

bool ShouldStaticMeshEvaluateWPOInRayTracing(FVector ViewCenter, const FBoxSphereBounds& Bounds)
{
	if (CVarRayTracingStaticMeshesWPOCulling.GetValueOnRenderThread() > 0)
	{
		const float CullingRadius = CVarRayTracingStaticMeshesWPOCullingRadius.GetValueOnRenderThread();

		if (FVector(ViewCenter - Bounds.Origin).Size() > (CullingRadius + Bounds.SphereRadius))
		{
			return false;
		}
	}

	return true;
}

/** Initialization constructor. */
FStaticMeshSceneProxy::FStaticMeshSceneProxy(UStaticMeshComponent* InComponent, bool bForceLODsShareStaticLighting)
	: FStaticMeshSceneProxy(FStaticMeshSceneProxyDesc(InComponent), bForceLODsShareStaticLighting)
{
}

/** Initialization constructor. */
FStaticMeshSceneProxy::FStaticMeshSceneProxy(const FStaticMeshSceneProxyDesc& InProxyDesc, bool bForceLODsShareStaticLighting)
	: FPrimitiveSceneProxy(InProxyDesc, InProxyDesc.GetStaticMesh()->GetFName())
	, RenderData(InProxyDesc.GetStaticMesh()->GetRenderData())
	, OverlayMaterial(InProxyDesc.GetOverlayMaterial())
	, OverlayMaterialMaxDrawDistance(InProxyDesc.GetOverlayMaterialMaxDrawDistance())
	, ForcedLodModel(InProxyDesc.ForcedLodModel)
	, bCastShadow(InProxyDesc.CastShadow)
	, bReverseCulling(InProxyDesc.bReverseCulling)
	, MaterialRelevance(InProxyDesc.GetMaterialRelevance(GetScene().GetShaderPlatform()))
	, WPODisableDistance(InProxyDesc.WorldPositionOffsetDisableDistance)
#if WITH_EDITORONLY_DATA
	, StreamingDistanceMultiplier(FMath::Max(0.0f, InProxyDesc.StreamingDistanceMultiplier))
	, StreamingTransformScale(InProxyDesc.TextureStreamingTransformScale)
	, MaterialStreamingRelativeBoxes(InProxyDesc.MaterialStreamingRelativeBoxes)
	, SectionIndexPreview(InProxyDesc.SectionIndexPreview)
	, MaterialIndexPreview(InProxyDesc.MaterialIndexPreview)
	, bPerSectionSelection(InProxyDesc.SelectedEditorSection != INDEX_NONE || InProxyDesc.SelectedEditorMaterial != INDEX_NONE)
#endif
	, StaticMesh(InProxyDesc.GetStaticMesh())
#if STATICMESH_ENABLE_DEBUG_RENDERING
	, Owner(InProxyDesc.GetOwner())
	, LightMapResolution(InProxyDesc.GetStaticLightMapResolution())
	, BodySetup(InProxyDesc.GetBodySetup())
	, CollisionTraceFlag(ECollisionTraceFlag::CTF_UseSimpleAndComplex)
	, CollisionResponse(InProxyDesc.GetCollisionResponseToChannels())
	, LODForCollision(InProxyDesc.GetStaticMesh()->LODForCollision)
	, bDrawMeshCollisionIfComplex(InProxyDesc.bDrawMeshCollisionIfComplex)
	, bDrawMeshCollisionIfSimple(InProxyDesc.bDrawMeshCollisionIfSimple)
#endif
{
	check(RenderData);
	checkf(RenderData->IsInitialized(), TEXT("Uninitialized Renderdata for Mesh: %s, Mesh NeedsLoad: %i, Mesh NeedsPostLoad: %i, Mesh Loaded: %i, Mesh NeedInit: %i, Mesh IsDefault: %i")
		, *StaticMesh->GetFName().ToString()
		, StaticMesh->HasAnyFlags(RF_NeedLoad)
		, StaticMesh->HasAnyFlags(RF_NeedPostLoad)
		, StaticMesh->HasAnyFlags(RF_LoadCompleted)
		, StaticMesh->HasAnyFlags(RF_NeedInitialization)
		, StaticMesh->HasAnyFlags(RF_ClassDefaultObject)
	);

	bIsStaticMesh = true;

	// Static meshes do not deform internally (save by material effects such as WPO and PDO, which is allowed).
	bHasDeformableMesh = false;

	// Static meshes can write to runtime virtual texture if they are set to do so.
	bSupportsRuntimeVirtualTexture = true;

	// True by constructors of proxy types whitch support gathering of UStreamableAssets
	bImplementsStreamableAssetGathering = true;

	bEvaluateWorldPositionOffset = InProxyDesc.bEvaluateWorldPositionOffset;

	const auto FeatureLevel = GetScene().GetFeatureLevel();
	const EShaderPlatform ShaderPlatform = GetScene().GetShaderPlatform();

	const int32 SMCurrentMinLOD = InProxyDesc.GetStaticMesh()->GetMinLODIdx();
	int32 EffectiveMinLOD = InProxyDesc.bOverrideMinLOD ? InProxyDesc.MinLOD : SMCurrentMinLOD;

	InProxyDesc.GetMaterialSlotsOverlayMaterial(MaterialSlotsOverlayMaterial);

#if WITH_EDITOR
	// If we plan to strip the min LOD during cooking, emulate that behavior in the editor
	static const auto* CVar = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.StaticMesh.StripMinLodDataDuringCooking"));
	check(CVar);
	if (CVar->GetValueOnAnyThread())
	{
		EffectiveMinLOD = FMath::Max<int32>(EffectiveMinLOD, SMCurrentMinLOD);
	}
#endif

#if PLATFORM_DESKTOP
	extern ENGINE_API int32 GUseMobileLODBiasOnDesktopES31;
	if (GUseMobileLODBiasOnDesktopES31 != 0 && FeatureLevel == ERHIFeatureLevel::ES3_1)
	{
		EffectiveMinLOD += InProxyDesc.GetStaticMesh()->GetRenderData()->LODBiasModifier;
	}
#endif

	bool bUsePSOPrecacheFallbackMaterial = InProxyDesc.UsePSOPrecacheFallbackMaterial();

	// Find the first LOD with any vertices (ie that haven't been stripped)
	int FirstAvailableLOD = 0;
	for (; FirstAvailableLOD < RenderData->LODResources.Num()-1; FirstAvailableLOD++)
	{
		if (RenderData->LODResources[FirstAvailableLOD].GetNumVertices() > 0)
		{
			break;
		}
	}

	if (GForceDefaultMaterial)
	{
		MaterialRelevance |= UMaterial::GetDefaultMaterial(MD_Surface)->GetRelevance(ShaderPlatform);
	}
	else if (bUsePSOPrecacheFallbackMaterial)
	{
		MaterialRelevance |= UPSOPrecacheSettingsManager::GetFallbackMaterial() ? UPSOPrecacheSettingsManager::GetFallbackMaterial()->GetRelevance(ShaderPlatform) : FMaterialRelevance();
	}

	check(FirstAvailableLOD < RenderData->LODResources.Num())

	ClampedMinLOD = FMath::Clamp(EffectiveMinLOD, FirstAvailableLOD, RenderData->LODResources.Num() - 1);

	SetWireframeColor(InProxyDesc.GetWireframeColor());

	// Copy the pointer to the volume data, async building of the data may modify the one on FStaticMeshLODResources while we are rendering
	DistanceFieldData = RenderData->LODResources[0].DistanceFieldData;
	CardRepresentationData = RenderData->LODResources[0].CardRepresentationData;

	bSupportsDistanceFieldRepresentation = MaterialRelevance.bOpaque && !MaterialRelevance.bUsesSkyMaterial && !MaterialRelevance.bUsesSingleLayerWaterMaterial && DistanceFieldData && DistanceFieldData->IsValid();
	bCastsDynamicIndirectShadow = InProxyDesc.bCastDynamicShadow && InProxyDesc.CastShadow && InProxyDesc.bCastDistanceFieldIndirectShadow && InProxyDesc.Mobility != EComponentMobility::Static && !InProxyDesc.bIsFirstPerson;
	DynamicIndirectShadowMinVisibility = FMath::Clamp(InProxyDesc.DistanceFieldIndirectShadowMinVisibility, 0.0f, 1.0f);
	DistanceFieldSelfShadowBias = FMath::Max(InProxyDesc.bOverrideDistanceFieldSelfShadowBias ? InProxyDesc.DistanceFieldSelfShadowBias : InProxyDesc.GetStaticMesh()->DistanceFieldSelfShadowBias, 0.0f);

	// Build the proxy's LOD data.
	bool bAnySectionCastsShadows = false;
	LODs.Empty(RenderData->LODResources.Num());
	const bool bLODsShareStaticLighting = RenderData->bLODsShareStaticLighting || bForceLODsShareStaticLighting;

	for (int32 LODIndex = 0; LODIndex < RenderData->LODResources.Num(); LODIndex++)
	{
		FLODInfo* NewLODInfo = new (LODs) FLODInfo(InProxyDesc, RenderData->LODVertexFactories, LODIndex, ClampedMinLOD, bLODsShareStaticLighting);

		// Under certain error conditions an LOD's material will be set to 
		// DefaultMaterial. Ensure our material view relevance is set properly.
		const int32 NumSections = NewLODInfo->Sections.Num();
		for (int32 SectionIndex = 0; SectionIndex < NumSections; ++SectionIndex)
		{
			const FLODInfo::FSectionInfo& SectionInfo = NewLODInfo->Sections[SectionIndex];
			bAnySectionCastsShadows |= RenderData->LODResources[LODIndex].Sections[SectionIndex].bCastShadow;
			if (SectionInfo.Material == UMaterial::GetDefaultMaterial(MD_Surface))
			{
				MaterialRelevance |= UMaterial::GetDefaultMaterial(MD_Surface)->GetRelevance(ShaderPlatform);
			}

			MaxWPOExtent = FMath::Max(MaxWPOExtent, SectionInfo.Material->GetMaxWorldPositionOffsetDisplacement());
		}
	}

	if (bUsePSOPrecacheFallbackMaterial)
	{
		OverlayMaterial = nullptr;

		//Null out the overlay material to avoid the extra drawing with the same result.
		FMaterialOverlayHelper::ForceMaterial(MaterialSlotsOverlayMaterial, nullptr);
	}

#if RHI_RAYTRACING
	bSupportRayTracing = IsRayTracingAllowed() && InProxyDesc.GetStaticMesh()->bSupportRayTracing;
	bHasRayTracingRepresentation = false;
	bDynamicRayTracingInstances = false;
	bDynamicRayTracingGeometry = false;
	bCreateDynamicRayTracingGeometries = true;

	if (bSupportRayTracing && !ensureMsgf(RenderData->RayTracingProxy, TEXT("StaticMesh [%s] missing RayTracingProxy."), StaticMesh ? *StaticMesh->GetName() : TEXT("None")))
	{
		bSupportRayTracing = false;
	}

	if (bSupportRayTracing)
	{
		const bool bWantsRayTracingWPO = bEvaluateWorldPositionOffset && MaterialRelevance.bUsesWorldPositionOffset && InProxyDesc.bEvaluateWorldPositionOffsetInRayTracing;

		if (bWantsRayTracingWPO)
		{
			// Need to use these temporary variables since compiler doesn't accept 'bitfield bool' as bool&
			bool bHasRayTracingRepresentationTmp;
			bool bDynamicRayTracingGeometryTmp;
			GetRayTracingWPOConfig(bHasRayTracingRepresentationTmp, bDynamicRayTracingGeometryTmp);

			bHasRayTracingRepresentation = bHasRayTracingRepresentationTmp;
			bDynamicRayTracingGeometry = bDynamicRayTracingGeometryTmp;
		}
		else
		{
			bHasRayTracingRepresentation = true;
		}

		// when ray tracing proxy uses rendering LODs we can use the FLODInfo for ray tracing
		// otherwise initialize RayTracingLODInfos to be used in GetRayTracingMeshElement(...)
		if (!RenderData->RayTracingProxy->bUsingRenderingLODs)
		{
			RayTracingLODInfos.Empty(RenderData->RayTracingProxy->LODs.Num());
			for (int32 LODIndex = 0; LODIndex < RenderData->RayTracingProxy->LODs.Num(); LODIndex++)
			{
				new (RayTracingLODInfos) FRayTracingLODInfo(InProxyDesc, LODIndex);
			}
		}
	}
#endif

	// WPO is typically used for ambient animations, so don't include in cached shadowmaps
	// Note mesh animation can also come from PDO or Tessellation but they are typically static uses so we ignore them for cached shadowmaps
	bGoodCandidateForCachedShadowmap = CacheShadowDepthsFromPrimitivesUsingWPO() || (!MaterialRelevance.bUsesWorldPositionOffset && !MaterialRelevance.bUsesDisplacement);

	// Disable shadow casting if no section has it enabled.
	bCastShadow = bCastShadow && bAnySectionCastsShadows;
	bCastDynamicShadow = bCastDynamicShadow && bCastShadow;

	bStaticElementsAlwaysUseProxyPrimitiveUniformBuffer = true;

	EnableGPUSceneSupportFlags();

#if STATICMESH_ENABLE_DEBUG_RENDERING
	// Setup Hierarchical LOD index
	enum HLODColors
	{
		HLODNone	= 0,	// Not part of a HLOD cluster (draw as white when visualizing)
		HLODChild	= 1,	// Part of a HLOD cluster but still a plain mesh
		HLOD0		= 2		// Colors for HLOD levels start at index 2
	};

	if (ALODActor* LODActorOwner = Cast<ALODActor>(Owner))
	{
		HierarchicalLODIndex = HLODColors::HLOD0 + LODActorOwner->LODLevel - 1; // ALODActor::LODLevel count from 1
	}
	else if (AWorldPartitionHLOD* WorldPartitionHLODOwner = Cast<AWorldPartitionHLOD>(Owner))
	{
		HierarchicalLODIndex = HLODColors::HLOD0 + WorldPartitionHLODOwner->GetLODLevel();
	}
	else if (InProxyDesc.GetLODParentPrimitive())
	{
		HierarchicalLODIndex = HLODColors::HLODChild;
	}
	else
	{
		HierarchicalLODIndex = HLODColors::HLODNone;
	}

	if (BodySetup)
	{
		CollisionTraceFlag = BodySetup->GetCollisionTraceFlag();
	}
#endif

	AddSpeedTreeWind();

	// Enable dynamic triangle reordering to remove/reduce sorting issue when rendered with a translucent material (i.e., order-independent-transparency)
	bSupportsSortedTriangles = InProxyDesc.bSortTriangles;

	if (IsAllowingApproximateOcclusionQueries())
	{
		bAllowApproximateOcclusion = true;
	}

	bOpaqueOrMasked = MaterialRelevance.bOpaque;
	bSupportsMaterialCache = MaterialRelevance.bSamplesMaterialCache;
	
	if (!MaterialRelevance.bUsesSingleLayerWaterMaterial)
	{
		UpdateVisibleInLumenScene();
	}

	MeshPaintTextureResource = InProxyDesc.GetMeshPaintTextureResource();
	MeshPaintTextureCoordinateIndex = InProxyDesc.MeshPaintTextureCoordinateIndex;

	if (InProxyDesc.ShouldCreateMaterialCacheProxy())
	{
		for (UMaterialCacheVirtualTexture* MaterialCacheTexture : InProxyDesc.MaterialCacheTextures)
		{
			if (FMaterialCacheVirtualTextureRenderProxy* RenderProxy = MaterialCacheTexture->CreateRenderProxy(
				InProxyDesc.MaterialCacheUVRegion,
				InProxyDesc.MaterialCacheTextureCoordinateIndex
			))
			{
				MaterialCacheRenderProxies.Emplace(RenderProxy);
			}
		}
		
		MaterialCacheDescriptor = PackMaterialCacheDescriptor(
			UINT16_MAX,
			InProxyDesc.MaterialCacheTextureCoordinateIndex
		);
	}
}

void FStaticMeshSceneProxy::SetEvaluateWorldPositionOffsetInRayTracing(FRHICommandListBase& RHICmdList, bool bEvaluate)
{
#if RHI_RAYTRACING
	if (!bSupportRayTracing)
	{
		return;
	}

	const bool bWantsRayTracingWPO = bEvaluateWorldPositionOffset && MaterialRelevance.bUsesWorldPositionOffset && bEvaluate;

	bool bNewDynamicRayTracingGeometry;
	if (bWantsRayTracingWPO)
	{
		// Need to use these temporary variables since compiler doesn't accept 'bitfield bool' as bool&
		bool bHasRayTracingRepresentationTmp;
		bool bDynamicRayTracingGeometryTmp;
		GetRayTracingWPOConfig(bHasRayTracingRepresentationTmp, bDynamicRayTracingGeometryTmp);

		bHasRayTracingRepresentation = bHasRayTracingRepresentationTmp;
		bNewDynamicRayTracingGeometry = bDynamicRayTracingGeometryTmp;
	}
	else
	{
		bHasRayTracingRepresentation = true;
		bNewDynamicRayTracingGeometry = false;
	}

	if (!bDynamicRayTracingGeometry && bNewDynamicRayTracingGeometry)
	{
		bDynamicRayTracingGeometry = bNewDynamicRayTracingGeometry;
		CreateDynamicRayTracingGeometries(RHICmdList);
	}
	else if (bDynamicRayTracingGeometry && !bNewDynamicRayTracingGeometry)
	{
		ReleaseDynamicRayTracingGeometries();
		bDynamicRayTracingGeometry = bNewDynamicRayTracingGeometry;
	}

	GetScene().UpdateCachedRayTracingState(this);
#endif
}

int32 FStaticMeshSceneProxy::GetLightMapCoordinateIndex() const
{
	const int32 LightMapCoordinateIndex = StaticMesh != nullptr ? StaticMesh->GetLightMapCoordinateIndex() : INDEX_NONE;
	return LightMapCoordinateIndex;
}

bool FStaticMeshSceneProxy::GetInstanceWorldPositionOffsetDisableDistance(float& OutWPODisableDistance) const
{
	OutWPODisableDistance = WPODisableDistance;
	return WPODisableDistance > 0.0f;
}

void FStaticMeshSceneProxy::GetStreamableRenderAssetInfo(const FBoxSphereBounds& InPrimitiveBounds, TArray<FStreamingRenderAssetPrimitiveInfo>& OutStreamableRenderAssets) const
{
	constexpr float ComponentScaling = 1.0f;
	const EMaterialQualityLevel::Type QualityLevel = GetCurrentMaterialQualityLevelChecked();
	for (const FLODInfo& LOD : LODs)
	{
		for (const FLODInfo::FSectionInfo Section : LOD.Sections)
		{
			if (const UMaterialInterface* ShadingMaterial = Section.Material)
			{
				const static FMeshUVChannelInfo Fallback(1.0f);
				// If no material assigned to mesh it will return null - lets use fallback in those cases
				const FMeshUVChannelInfo* UVChannelData = StaticMesh->GetUVChannelData(Section.MaterialIndex);

				const FMeshUVChannelInfo* UsedUvChannelData = UVChannelData ? UVChannelData : &Fallback;

				if (TSharedPtr<const FMaterialCachedTexturesSamplingInfo> CachedSamplingInfo = ShadingMaterial->GetCachedTexturesSamplingInfo())
				{
					CachedSamplingInfo->GetStreamingRenderAssetPrimitiveInfo(InPrimitiveBounds, ComponentScaling, *UsedUvChannelData, QualityLevel, OutStreamableRenderAssets);
				}
				else
				{
					// Unexpected - Cache unavailable
					// Skip this material, the streamer won't see this material's textures.
					ensureMsgf(IsRunningCookCommandlet(), TEXT("No cached texture sampling info for material %s; skipping its textures in the streaming report."), *ShadingMaterial->GetName());
				}
			}
		}

		if (const FLightMap* LightMap = LOD.GetLightMap())
		{
			if (const FLightMap2D* LightMap2D = LightMap->GetLightMap2D())
			{
				uint32 LightMapIndex = AllowHighQualityLightmaps(GetScene().GetFeatureLevel()) ? 0 : 1;
				const FVector2D& Scale = LightMap2D->GetCoordinateScale();
				if (LightMap2D->IsValid(LightMapIndex) && Scale.X > UE_SMALL_NUMBER && Scale.Y > UE_SMALL_NUMBER)
				{
					const float TexelFactor = StaticMesh->GetLightmapUVDensity() / FMath::Min(Scale.X, Scale.Y);
					OutStreamableRenderAssets.Add(FStreamingRenderAssetPrimitiveInfo{const_cast<UTexture2D*>(LightMap2D->GetTexture(LightMapIndex)), InPrimitiveBounds, TexelFactor, PackedRelativeBox_Identity});
					OutStreamableRenderAssets.Add(FStreamingRenderAssetPrimitiveInfo{LightMap2D->GetAOMaterialMaskTexture(), InPrimitiveBounds, TexelFactor, PackedRelativeBox_Identity});
					OutStreamableRenderAssets.Add(FStreamingRenderAssetPrimitiveInfo{LightMap2D->GetSkyOcclusionTexture(), InPrimitiveBounds, TexelFactor, PackedRelativeBox_Identity});
				}
			}
		}

		if (const FShadowMap* ShadowMap = LOD.GetShadowMap())
		{
			if (const FShadowMap2D* ShadowMap2D = ShadowMap->GetShadowMap2D())
			{
				const FVector2D& Scale = ShadowMap2D->GetCoordinateScale();
				const float TexelFactor = StaticMesh->GetLightmapUVDensity() / FMath::Min(Scale.X, Scale.Y);
				if (Scale.X > UE_SMALL_NUMBER && Scale.Y > UE_SMALL_NUMBER)
				{
					OutStreamableRenderAssets.Add(FStreamingRenderAssetPrimitiveInfo{const_cast<UTexture2D*>(ShadowMap2D->GetTexture()), InPrimitiveBounds, TexelFactor, PackedRelativeBox_Identity});
				}
			}
		}
	}
	
	if (StaticMesh->RenderResourceSupportsStreaming() && (StaticMesh->GetRenderAssetType() == EStreamableRenderAssetType::StaticMesh))
	{
		const float TexelFactor = ForcedLodModel > 0 ? - (StaticMesh->GetRenderData()->LODResources.Num() - ForcedLodModel + 1) : InPrimitiveBounds.SphereRadius * 2.0f;
		OutStreamableRenderAssets.Add(FStreamingRenderAssetPrimitiveInfo(const_cast<UStaticMesh*>(StaticMesh), InPrimitiveBounds, TexelFactor, PackedRelativeBox_Identity, true, false));
	}
}

FStaticMeshSceneProxy::~FStaticMeshSceneProxy()
{
}

void FStaticMeshSceneProxy::AddSpeedTreeWind()
{
	if (StaticMesh && RenderData && StaticMesh->SpeedTreeWind.IsValid())
	{
		for (int32 LODIndex = 0; LODIndex < RenderData->LODVertexFactories.Num(); ++LODIndex)
		{
			GetScene().AddSpeedTreeWind(&RenderData->LODVertexFactories[LODIndex].VertexFactory, StaticMesh);
			GetScene().AddSpeedTreeWind(&RenderData->LODVertexFactories[LODIndex].VertexFactoryOverrideColorVertexBuffer, StaticMesh);
		}
	}
}

void FStaticMeshSceneProxy::RemoveSpeedTreeWind()
{
	if (StaticMesh && RenderData && StaticMesh->SpeedTreeWind.IsValid())
	{
		for (int32 LODIndex = 0; LODIndex < RenderData->LODVertexFactories.Num(); ++LODIndex)
		{
			GetScene().RemoveSpeedTreeWind_RenderThread(&RenderData->LODVertexFactories[LODIndex].VertexFactoryOverrideColorVertexBuffer, StaticMesh);
			GetScene().RemoveSpeedTreeWind_RenderThread(&RenderData->LODVertexFactories[LODIndex].VertexFactory, StaticMesh);
		}
	}
}

bool UStaticMeshComponent::SetLODDataCount( const uint32 MinSize, const uint32 MaxSize )
{
	check(MaxSize <= MAX_STATIC_MESH_LODS);

	if (MaxSize < (uint32)LODData.Num())
	{
		// FStaticMeshComponentLODInfo can't be deleted directly as it has rendering resources
		for (int32 Index = MaxSize; Index < LODData.Num(); Index++)
		{
			LODData[Index].ReleaseOverrideVertexColorsAndBlock();
		}

		// call destructors
		LODData.RemoveAt(MaxSize, LODData.Num() - MaxSize);
		return true;
	}
	
	if(MinSize > (uint32)LODData.Num())
	{
		// call constructors
		LODData.Reserve(MinSize);

		// TArray doesn't have a function for constructing n items
		uint32 ItemCountToAdd = MinSize - LODData.Num();
		for(uint32 i = 0; i < ItemCountToAdd; ++i)
		{
			// call constructor
			new (LODData)FStaticMeshComponentLODInfo(this);
		}
		return true;
	}

	return false;
}

SIZE_T FStaticMeshSceneProxy::GetTypeHash() const
{
	static size_t UniquePointer;
	return reinterpret_cast<size_t>(&UniquePointer);
}

bool FStaticMeshSceneProxy::GetShadowMeshElement(int32 LODIndex, int32 BatchIndex, uint8 InDepthPriorityGroup, FMeshBatch& OutMeshBatch, bool bDitheredLODTransition) const
{
	const FStaticMeshLODResources& LOD = RenderData->LODResources[LODIndex];
	const FStaticMeshVertexFactories& VFs = RenderData->LODVertexFactories[LODIndex];
	const FLODInfo& ProxyLODInfo = LODs[LODIndex];

	const bool bUseReversedIndices = GUseReversedIndexBuffer &&
		ShouldRenderBackFaces() &&
		LOD.bHasReversedDepthOnlyIndices;
	const bool bNoIndexBufferAvailable = !bUseReversedIndices && !LOD.bHasDepthOnlyIndices;

	if (bNoIndexBufferAvailable)
	{
		return false;
	}

	FMeshBatchElement& OutMeshBatchElement = OutMeshBatch.Elements[0];

	if (ProxyLODInfo.OverrideColorVertexBuffer)
	{
		OutMeshBatch.VertexFactory = &VFs.VertexFactoryOverrideColorVertexBuffer;
		OutMeshBatchElement.VertexFactoryUserData = ProxyLODInfo.OverrideColorVFUniformBuffer.GetReference();
	}
	else
	{
		OutMeshBatch.VertexFactory = &VFs.VertexFactory;
		OutMeshBatchElement.VertexFactoryUserData = VFs.VertexFactory.GetUniformBuffer();
	}

	OutMeshBatchElement.IndexBuffer = LOD.AdditionalIndexBuffers && bUseReversedIndices ? &LOD.AdditionalIndexBuffers->ReversedDepthOnlyIndexBuffer : &LOD.DepthOnlyIndexBuffer;
	OutMeshBatchElement.FirstIndex = 0;
	OutMeshBatchElement.NumPrimitives = LOD.DepthOnlyNumTriangles;
	OutMeshBatchElement.MinVertexIndex = 0;
	OutMeshBatchElement.MaxVertexIndex = LOD.VertexBuffers.PositionVertexBuffer.GetNumVertices() - 1;

	OutMeshBatch.LODIndex = LODIndex;
#if STATICMESH_ENABLE_DEBUG_RENDERING
	OutMeshBatch.VisualizeLODIndex = LODIndex;
	OutMeshBatch.VisualizeHLODIndex = HierarchicalLODIndex;
#endif
	OutMeshBatch.ReverseCulling = IsReversedCullingNeeded(bUseReversedIndices);
	OutMeshBatch.Type = PT_TriangleList;
	OutMeshBatch.DepthPriorityGroup = InDepthPriorityGroup;
	OutMeshBatch.LCI = &ProxyLODInfo;
	OutMeshBatch.MaterialRenderProxy = UMaterial::GetDefaultMaterial(MD_Surface)->GetRenderProxy();

	// By default this will be a shadow only mesh.
	OutMeshBatch.bUseForMaterial = false;
	OutMeshBatch.bUseForDepthPass = false;
	OutMeshBatch.bUseAsOccluder = false;

	SetMeshElementScreenSize(LODIndex, bDitheredLODTransition, OutMeshBatch);

	return true;
}

/** Sets up a FMeshBatch for a specific LOD and element. */
bool FStaticMeshSceneProxy::GetMeshElement(
	int32 LODIndex, 
	int32 BatchIndex, 
	int32 SectionIndex, 
	uint8 InDepthPriorityGroup, 
	bool bUseSelectionOutline,
	bool bAllowPreCulledIndices, 
	FMeshBatch& OutMeshBatch) const
{
	const ERHIFeatureLevel::Type FeatureLevel = GetScene().GetFeatureLevel();
	const FStaticMeshLODResources& LOD = RenderData->LODResources[LODIndex];
	const FStaticMeshVertexFactories& VFs = RenderData->LODVertexFactories[LODIndex];
	const FStaticMeshSection& Section = LOD.Sections[SectionIndex];

	if (Section.NumTriangles == 0)
	{
		return false;
	}

	const FLODInfo& ProxyLODInfo = LODs[LODIndex];

	UMaterialInterface* MaterialInterface = ProxyLODInfo.Sections[SectionIndex].Material;
	const FMaterialRenderProxy* MaterialRenderProxy = MaterialInterface->GetRenderProxy();
	const FMaterial& Material = MaterialRenderProxy->GetIncompleteMaterialWithFallback(FeatureLevel);

	const FVertexFactory* VertexFactory = nullptr;

	FMeshBatchElement& OutMeshBatchElement = OutMeshBatch.Elements[0];

#if WITH_EDITORONLY_DATA
	// If material is hidden, then skip the draw.
	if ((MaterialIndexPreview >= 0) && (MaterialIndexPreview != Section.MaterialIndex))
	{
		return false;
	}
	// If section is hidden, then skip the draw.
	if ((SectionIndexPreview >= 0) && (SectionIndexPreview != SectionIndex))
	{
		return false;
	}

	OutMeshBatch.bUseSelectionOutline = bPerSectionSelection ? bUseSelectionOutline : true;
#endif

	// Has the mesh component overridden the vertex color stream for this mesh LOD?
	if (ProxyLODInfo.OverrideColorVertexBuffer)
	{
		// Make sure the indices are accessing data within the vertex buffer's
		check(Section.MaxVertexIndex < ProxyLODInfo.OverrideColorVertexBuffer->GetNumVertices())

		// Use the instanced colors vertex factory.
		VertexFactory = &VFs.VertexFactoryOverrideColorVertexBuffer;

		OutMeshBatchElement.VertexFactoryUserData = ProxyLODInfo.OverrideColorVFUniformBuffer.GetReference();
		OutMeshBatchElement.UserData = ProxyLODInfo.OverrideColorVertexBuffer;
		OutMeshBatchElement.bUserDataIsColorVertexBuffer = true;
	}
	else
	{
		VertexFactory = &VFs.VertexFactory;

		OutMeshBatchElement.VertexFactoryUserData = VFs.VertexFactory.GetUniformBuffer();
	}

	const bool bWireframe = false;

	// Determine based on the primitive option to reverse culling and current scale if we want to use reversed indices.
	// Two sided material use bIsFrontFace which is wrong with Reversed Indices.
	const bool bUseReversedIndices = GUseReversedIndexBuffer &&
		ShouldRenderBackFaces() &&
		(LOD.bHasReversedIndices != 0) &&
		!Material.IsTwoSided();

	// No support for stateless dithered LOD transitions for movable meshes
	const bool bDitheredLODTransition = !IsMovable() && Material.IsDitheredLODTransition();

	const uint32 NumPrimitives = SetMeshElementGeometrySource(Section, LOD.IndexBuffer, LOD.AdditionalIndexBuffers, VertexFactory, bWireframe, bUseReversedIndices, OutMeshBatch);

	if (NumPrimitives > 0)
	{
		OutMeshBatch.SegmentIndex = SectionIndex;
		OutMeshBatch.MeshIdInPrimitive = SectionIndex;

		OutMeshBatch.LODIndex = LODIndex;
#if STATICMESH_ENABLE_DEBUG_RENDERING
		OutMeshBatch.VisualizeLODIndex = LODIndex;
		OutMeshBatch.VisualizeHLODIndex = HierarchicalLODIndex;
#endif
		OutMeshBatch.ReverseCulling = IsReversedCullingNeeded(bUseReversedIndices);
		OutMeshBatch.CastShadow = bCastShadow && Section.bCastShadow;
#if RHI_RAYTRACING
		OutMeshBatch.CastRayTracedShadow = OutMeshBatch.CastShadow && bCastDynamicShadow;
#endif
		OutMeshBatch.DepthPriorityGroup = (ESceneDepthPriorityGroup)InDepthPriorityGroup;
		OutMeshBatch.LCI = &ProxyLODInfo;
		OutMeshBatch.MaterialRenderProxy = MaterialRenderProxy;

		OutMeshBatchElement.MinVertexIndex = Section.MinVertexIndex;
		OutMeshBatchElement.MaxVertexIndex = Section.MaxVertexIndex;
#if STATICMESH_ENABLE_DEBUG_RENDERING
		OutMeshBatchElement.VisualizeElementIndex = SectionIndex;
#endif

		SetMeshElementScreenSize(LODIndex, bDitheredLODTransition, OutMeshBatch);

		return true;
	}
	else
	{
		return false;
	}
}

bool FStaticMeshSceneProxy::GetRayTracingMeshElement(int32 LODIndex, int32 BatchIndex, int32 SectionIndex, uint8 InDepthPriorityGroup, FMeshBatch& OutMeshBatch) const
{
	const ERHIFeatureLevel::Type FeatureLevel = GetScene().GetFeatureLevel();
	const FStaticMeshRayTracingProxyLOD& LOD = RenderData->RayTracingProxy->LODs[LODIndex];
	const FStaticMeshVertexFactories& VFs = (*RenderData->RayTracingProxy->LODVertexFactories)[LODIndex];
	const FStaticMeshSection& Section = (*LOD.Sections)[SectionIndex];

	if (Section.NumTriangles == 0)
	{
		return false;
	}

	const FRayTracingLODInfo& LODInfo = RayTracingLODInfos[LODIndex];

	UMaterialInterface* MaterialInterface = LODInfo.Sections[SectionIndex].Material;
	const FMaterialRenderProxy* MaterialRenderProxy = MaterialInterface->GetRenderProxy();
	const FMaterial& Material = MaterialRenderProxy->GetIncompleteMaterialWithFallback(FeatureLevel);

	const FVertexFactory* VertexFactory = &VFs.VertexFactory;

	FMeshBatchElement& OutMeshBatchElement = OutMeshBatch.Elements[0];
	OutMeshBatchElement.VertexFactoryUserData = VFs.VertexFactory.GetUniformBuffer();

	const bool bWireframe = false;
	const bool bUseReversedIndices = false;

	// No support for stateless dithered LOD transitions for movable meshes
	const bool bDitheredLODTransition = !IsMovable() && Material.IsDitheredLODTransition();

	const uint32 NumPrimitives = SetMeshElementGeometrySource(Section, *LOD.IndexBuffer, nullptr, VertexFactory, bWireframe, bUseReversedIndices, OutMeshBatch);

	if (NumPrimitives > 0)
	{
		OutMeshBatch.SegmentIndex = SectionIndex;
		OutMeshBatch.MeshIdInPrimitive = SectionIndex;

		OutMeshBatch.LODIndex = LODIndex;
		OutMeshBatch.ReverseCulling = IsReversedCullingNeeded(bUseReversedIndices);
		OutMeshBatch.CastShadow = bCastShadow && Section.bCastShadow;
#if RHI_RAYTRACING
		OutMeshBatch.CastRayTracedShadow = OutMeshBatch.CastShadow && bCastDynamicShadow;
#endif
		OutMeshBatch.DepthPriorityGroup = (ESceneDepthPriorityGroup)InDepthPriorityGroup;
		OutMeshBatch.LCI = nullptr;
		OutMeshBatch.MaterialRenderProxy = MaterialRenderProxy;

		OutMeshBatchElement.MinVertexIndex = Section.MinVertexIndex;
		OutMeshBatchElement.MaxVertexIndex = Section.MaxVertexIndex;

		SetMeshElementScreenSize(LODIndex, bDitheredLODTransition, OutMeshBatch);

		return true;
	}
	else
	{
		return false;
	}
}

#if RHI_RAYTRACING

void FStaticMeshSceneProxy::GetRayTracingWPOConfig(bool& bOutHasRayTracingRepresentation, bool& bOutDynamicRayTracingGeometry) const
{
	GetStaticMeshRayTracingWPOConfig(bOutHasRayTracingRepresentation, bOutDynamicRayTracingGeometry);
}

bool FStaticMeshSceneProxy::ShouldEvaluateWPOInRayTracing(FVector ViewCenter) const
{
	return ShouldStaticMeshEvaluateWPOInRayTracing(ViewCenter, GetBounds());
}

void FStaticMeshSceneProxy::CreateDynamicRayTracingGeometries(FRHICommandListBase& RHICmdList)
{
	if (!bCreateDynamicRayTracingGeometries)
	{
		return;
	}

	check(bDynamicRayTracingGeometry);
	check(DynamicRayTracingGeometries.IsEmpty());

	FStaticMeshRayTracingProxyLODArray& RayTracingLODs = RenderData->RayTracingProxy->LODs;

	DynamicRayTracingGeometries.AddDefaulted(RayTracingLODs.Num());

	for (int32 LODIndex = 0; LODIndex < RayTracingLODs.Num(); LODIndex++)
	{
		FRayTracingGeometryInitializer Initializer = RayTracingLODs[LODIndex].RayTracingGeometry->GetInitializer();
		for (FRayTracingGeometrySegment& Segment : Initializer.Segments)
		{
			Segment.VertexBuffer = nullptr;
		}
		Initializer.bAllowUpdate = true;
		Initializer.bFastBuild = true;
		Initializer.Type = ERayTracingGeometryInitializerType::Rendering;

		DynamicRayTracingGeometries[LODIndex].SetInitializer(MoveTemp(Initializer));
		DynamicRayTracingGeometries[LODIndex].InitResource(RHICmdList);
	}
}

void FStaticMeshSceneProxy::ReleaseDynamicRayTracingGeometries()
{
	if (!bCreateDynamicRayTracingGeometries)
	{
		return;
	}

	checkf(DynamicRayTracingGeometries.IsEmpty() || bDynamicRayTracingGeometry, TEXT("Proxy shouldn't have DynamicRayTracingGeometries since bDynamicRayTracingGeometry is false."));

	for (auto& Geometry : DynamicRayTracingGeometries)
	{
		Geometry.ReleaseResource();
	}

	DynamicRayTracingGeometries.Empty();
}

#endif // RHI_RAYTRACING

void FStaticMeshSceneProxy::CreateRenderThreadResources(FRHICommandListBase& RHICmdList)
{
#if RHI_RAYTRACING
	if (IsRayTracingAllowed())
	{
		// copy RayTracingGeometryGroupHandle from FStaticMeshRenderData since UStaticMesh can be released before the proxy is destroyed
		RayTracingGeometryGroupHandle = RenderData->RayTracingGeometryGroupHandle;
	}

	if(IsRayTracingEnabled() && bDynamicRayTracingGeometry)
	{
		CreateDynamicRayTracingGeometries(RHICmdList);
	}
	else
	{
		checkf(DynamicRayTracingGeometries.IsEmpty(), TEXT("Proxy shouldn't have entries in DynamicRayTracingGeometries."));
	}
#endif

	MeshPaintTextureDescriptor = MeshPaintVirtualTexture::GetTextureDescriptor(MeshPaintTextureResource, MeshPaintTextureCoordinateIndex);
}

void FStaticMeshSceneProxy::DestroyRenderThreadResources()
{
	FPrimitiveSceneProxy::DestroyRenderThreadResources();

#if RHI_RAYTRACING
	ReleaseDynamicRayTracingGeometries();
#endif

	// Call here because it uses RenderData from the StaticMesh which is not guaranteed to still be valid after this DestroyRenderThreadResources call
	RemoveSpeedTreeWind();
	StaticMesh = nullptr;
}

/** Sets up a wireframe FMeshBatch for a specific LOD. */
bool FStaticMeshSceneProxy::GetWireframeMeshElement(int32 LODIndex, int32 BatchIndex, const FMaterialRenderProxy* WireframeRenderProxy, uint8 InDepthPriorityGroup, bool bAllowPreCulledIndices, FMeshBatch& OutMeshBatch) const
{
	const FStaticMeshLODResources& LODModel = RenderData->LODResources[LODIndex];
	const FStaticMeshVertexFactories& VFs = RenderData->LODVertexFactories[LODIndex];
	const FLODInfo& ProxyLODInfo = LODs[LODIndex];
	const FVertexFactory* VertexFactory = nullptr;

	FMeshBatchElement& OutBatchElement = OutMeshBatch.Elements[0];

	if (ProxyLODInfo.OverrideColorVertexBuffer)
	{
		VertexFactory = &VFs.VertexFactoryOverrideColorVertexBuffer;

		OutBatchElement.VertexFactoryUserData = ProxyLODInfo.OverrideColorVFUniformBuffer.GetReference();
	}
	else
	{
		VertexFactory = &VFs.VertexFactory;

		OutBatchElement.VertexFactoryUserData = VFs.VertexFactory.GetUniformBuffer();
	}

	const bool bWireframe = true;
	const bool bUseReversedIndices = false;
	const bool bDitheredLODTransition = false;

	OutMeshBatch.ReverseCulling = IsReversedCullingNeeded(bUseReversedIndices);
	OutMeshBatch.CastShadow = bCastShadow;
	OutMeshBatch.DepthPriorityGroup = (ESceneDepthPriorityGroup)InDepthPriorityGroup;
	OutMeshBatch.MaterialRenderProxy = WireframeRenderProxy;

	OutBatchElement.MinVertexIndex = 0;
	OutBatchElement.MaxVertexIndex = LODModel.GetNumVertices() - 1;

	const uint32_t NumPrimitives = SetMeshElementGeometrySource(LODModel.Sections[0], LODModel.IndexBuffer, LODModel.AdditionalIndexBuffers, VertexFactory, bWireframe, bUseReversedIndices, OutMeshBatch);
	SetMeshElementScreenSize(LODIndex, bDitheredLODTransition, OutMeshBatch);

	return NumPrimitives > 0;
}

bool FStaticMeshSceneProxy::GetCollisionMeshElement(
	int32 LODIndex,
	int32 BatchIndex,
	int32 SectionIndex,
	uint8 InDepthPriorityGroup,
	const FMaterialRenderProxy* RenderProxy,
	FMeshBatch& OutMeshBatch) const
{
	const FStaticMeshLODResources& LOD = RenderData->LODResources[LODIndex];
	const FStaticMeshVertexFactories& VFs = RenderData->LODVertexFactories[LODIndex];
	const FStaticMeshSection& Section = LOD.Sections[SectionIndex];

	if (Section.NumTriangles == 0)
	{
		return false;
	}

	const FVertexFactory* VertexFactory = nullptr;

	const FLODInfo& ProxyLODInfo = LODs[LODIndex];

	const bool bWireframe = false;
	const bool bUseReversedIndices = false;
	const bool bAllowPreCulledIndices = true;
	const bool bDitheredLODTransition = false;

	SetMeshElementGeometrySource(Section, LOD.IndexBuffer, LOD.AdditionalIndexBuffers, VertexFactory, bWireframe, bUseReversedIndices, OutMeshBatch);

	FMeshBatchElement& OutMeshBatchElement = OutMeshBatch.Elements[0];

	if (ProxyLODInfo.OverrideColorVertexBuffer)
	{
		VertexFactory = &VFs.VertexFactoryOverrideColorVertexBuffer;

		OutMeshBatchElement.VertexFactoryUserData = ProxyLODInfo.OverrideColorVFUniformBuffer.GetReference();
	}
	else
	{
		VertexFactory = &VFs.VertexFactory;

		OutMeshBatchElement.VertexFactoryUserData = VFs.VertexFactory.GetUniformBuffer();
	}

	if (OutMeshBatchElement.NumPrimitives > 0)
	{
		OutMeshBatch.LODIndex = LODIndex;
#if STATICMESH_ENABLE_DEBUG_RENDERING
		OutMeshBatch.VisualizeLODIndex = LODIndex;
		OutMeshBatch.VisualizeHLODIndex = HierarchicalLODIndex;
#endif
		OutMeshBatch.ReverseCulling = IsReversedCullingNeeded(bUseReversedIndices);
		OutMeshBatch.CastShadow = false;
		OutMeshBatch.DepthPriorityGroup = (ESceneDepthPriorityGroup)InDepthPriorityGroup;
		OutMeshBatch.LCI = &ProxyLODInfo;
		OutMeshBatch.VertexFactory = VertexFactory;
		OutMeshBatch.MaterialRenderProxy = RenderProxy;

		OutMeshBatchElement.MinVertexIndex = Section.MinVertexIndex;
		OutMeshBatchElement.MaxVertexIndex = Section.MaxVertexIndex;
#if STATICMESH_ENABLE_DEBUG_RENDERING
		OutMeshBatchElement.VisualizeElementIndex = SectionIndex;
#endif

		SetMeshElementScreenSize(LODIndex, bDitheredLODTransition, OutMeshBatch);

		return true;
	}
	else
	{
		return false;
	}
}

#if WITH_EDITORONLY_DATA

bool FStaticMeshSceneProxy::GetPrimitiveDistance(int32 LODIndex, int32 SectionIndex, const FVector& ViewOrigin, float& PrimitiveDistance) const
{
	const bool bUseNewMetrics = CVarStreamingUseNewMetrics.GetValueOnRenderThread() != 0;
	const float OneOverDistanceMultiplier = 1.f / FMath::Max<float>(UE_SMALL_NUMBER, StreamingDistanceMultiplier);

	if (bUseNewMetrics && LODs.IsValidIndex(LODIndex) && LODs[LODIndex].Sections.IsValidIndex(SectionIndex))
	{
		// The LOD-section data is stored per material index as it is only used for texture streaming currently.
		const int32 MaterialIndex = LODs[LODIndex].Sections[SectionIndex].MaterialIndex;

		if (MaterialStreamingRelativeBoxes.IsValidIndex(MaterialIndex))
		{
			FBoxSphereBounds MaterialBounds;
			UnpackRelativeBox(GetBounds(), MaterialStreamingRelativeBoxes[MaterialIndex], MaterialBounds);

			FVector ViewToObject = (MaterialBounds.Origin - ViewOrigin).GetAbs();
			FVector BoxViewToObject = ViewToObject.ComponentMin(MaterialBounds.BoxExtent);
			float DistSq = FVector::DistSquared(BoxViewToObject, ViewToObject);

			PrimitiveDistance = FMath::Sqrt(FMath::Max<float>(1.f, DistSq)) * OneOverDistanceMultiplier;
			return true;
		}
	}

	if (FPrimitiveSceneProxy::GetPrimitiveDistance(LODIndex, SectionIndex, ViewOrigin, PrimitiveDistance))
	{
		PrimitiveDistance *= OneOverDistanceMultiplier;
		return true;
	}
	return false;
}

bool FStaticMeshSceneProxy::GetMeshUVDensities(int32 LODIndex, int32 SectionIndex, FVector4& WorldUVDensities) const
{
	if (LODs.IsValidIndex(LODIndex) && LODs[LODIndex].Sections.IsValidIndex(SectionIndex))
	{
		// The LOD-section data is stored per material index as it is only used for texture streaming currently.
		const int32 MaterialIndex = LODs[LODIndex].Sections[SectionIndex].MaterialIndex;

		 if (RenderData->UVChannelDataPerMaterial.IsValidIndex(MaterialIndex))
		 {
			const FMeshUVChannelInfo& UVChannelData = RenderData->UVChannelDataPerMaterial[MaterialIndex];

			WorldUVDensities.Set(
				UVChannelData.LocalUVDensities[0] * StreamingTransformScale,
				UVChannelData.LocalUVDensities[1] * StreamingTransformScale,
				UVChannelData.LocalUVDensities[2] * StreamingTransformScale,
				UVChannelData.LocalUVDensities[3] * StreamingTransformScale);

			return true;
		 }
	}
	return FPrimitiveSceneProxy::GetMeshUVDensities(LODIndex, SectionIndex, WorldUVDensities);
}

bool FStaticMeshSceneProxy::GetMaterialTextureScales(int32 LODIndex, int32 SectionIndex, const FMaterialRenderProxy* MaterialRenderProxy, FVector4f* OneOverScales, FIntVector4* UVChannelIndices) const
{
	if (LODs.IsValidIndex(LODIndex) && LODs[LODIndex].Sections.IsValidIndex(SectionIndex))
	{
		const UMaterialInterface* Material = LODs[LODIndex].Sections[SectionIndex].Material;
		if (Material)
		{
			// This is thread safe because material texture data is only updated while the renderthread is idle.
			for (const FMaterialTextureInfo& TextureData : Material->GetTextureStreamingData())
			{
				const int32 TextureIndex = TextureData.TextureIndex;
				if (TextureData.IsValid(true))
				{
					OneOverScales[TextureIndex / 4][TextureIndex % 4] = 1.f / TextureData.SamplingScale;
					UVChannelIndices[TextureIndex / 4][TextureIndex % 4] = TextureData.UVChannelIndex;
				}
			}
			for (const FMaterialTextureInfo& TextureData : Material->TextureStreamingDataMissingEntries)
			{
				const int32 TextureIndex = TextureData.TextureIndex;
				if (TextureIndex >= 0 && TextureIndex < TEXSTREAM_MAX_NUM_TEXTURES_PER_MATERIAL)
				{
					OneOverScales[TextureIndex / 4][TextureIndex % 4] = 1.f;
					UVChannelIndices[TextureIndex / 4][TextureIndex % 4] = 0;
				}
			}
			return true;
		}
	}
	return false;
}
#endif

uint32 FStaticMeshSceneProxy::SetMeshElementGeometrySource(
	const FStaticMeshSection& Section,
	const FRawStaticIndexBuffer& IndexBuffer,
	const FAdditionalStaticMeshIndexBuffers* AdditionalIndexBuffers,
	const FVertexFactory* VertexFactory,
	bool bWireframe,
	bool bUseReversedIndices,
	FMeshBatch& OutMeshBatch) const
{
	if (Section.NumTriangles == 0)
	{
		return 0;
	}

	FMeshBatchElement& OutMeshBatchElement = OutMeshBatch.Elements[0];
	uint32 NumPrimitives = 0;

	if (bWireframe)
	{
		if (AdditionalIndexBuffers && AdditionalIndexBuffers->WireframeIndexBuffer.IsInitialized())
		{
			OutMeshBatch.Type = PT_LineList;
			OutMeshBatchElement.FirstIndex = 0;
			OutMeshBatchElement.IndexBuffer = &AdditionalIndexBuffers->WireframeIndexBuffer;
			NumPrimitives = AdditionalIndexBuffers->WireframeIndexBuffer.GetNumIndices() / 2;
		}
		else
		{
			OutMeshBatch.Type = PT_TriangleList;

			OutMeshBatchElement.FirstIndex = 0;
			OutMeshBatchElement.IndexBuffer = &IndexBuffer;
			NumPrimitives = IndexBuffer.GetNumIndices() / 3;

			OutMeshBatch.bWireframe = true;
			OutMeshBatch.bDisableBackfaceCulling = true;
		}
	}
	else
	{
		OutMeshBatch.Type = PT_TriangleList;

		OutMeshBatchElement.IndexBuffer = bUseReversedIndices ? &AdditionalIndexBuffers->ReversedIndexBuffer : &IndexBuffer;
		OutMeshBatchElement.FirstIndex = Section.FirstIndex;
		NumPrimitives = Section.NumTriangles;
	}

	OutMeshBatchElement.NumPrimitives = NumPrimitives;
	OutMeshBatch.VertexFactory = VertexFactory;

	return NumPrimitives;
}

uint32 FStaticMeshSceneProxy::SetMeshElementGeometrySource(
	int32 LODIndex,
	int32 SectionIndex,
	bool bWireframe,
	bool bUseReversedIndices,
	bool bAllowPreCulledIndices, // unused
	const FVertexFactory* VertexFactory,
	FMeshBatch& OutMeshBatch) const
{
	const FStaticMeshLODResources& LODModel = RenderData->LODResources[LODIndex];

	const FStaticMeshSection& Section = LODModel.Sections[SectionIndex];

	return SetMeshElementGeometrySource(Section, LODModel.IndexBuffer, LODModel.AdditionalIndexBuffers, VertexFactory, bWireframe, bUseReversedIndices, OutMeshBatch);
}

void FStaticMeshSceneProxy::SetMeshElementScreenSize(int32 LODIndex, bool bDitheredLODTransition, FMeshBatch& OutMeshBatch) const
{
	FMeshBatchElement& OutBatchElement = OutMeshBatch.Elements[0];

	if (ForcedLodModel > 0)
	{
		OutMeshBatch.bDitheredLODTransition = false;

		OutBatchElement.MaxScreenSize = 0.0f;
		OutBatchElement.MinScreenSize = -1.0f;
	}
	else
	{
		OutMeshBatch.bDitheredLODTransition = bDitheredLODTransition;

		OutBatchElement.MaxScreenSize = GetScreenSize(LODIndex);
		OutBatchElement.MinScreenSize = 0.0f;
		if (LODIndex < MAX_STATIC_MESH_LODS - 1)
		{
			OutBatchElement.MinScreenSize = GetScreenSize(LODIndex + 1);
		}
	}
}

bool FStaticMeshSceneProxy::ShouldRenderBackFaces() const
{
	// Use != to ensure consistent face direction between negatively and positively scaled primitives
	return bReverseCulling != IsLocalToWorldDeterminantNegative();
}

bool FStaticMeshSceneProxy::IsReversedCullingNeeded(bool bUseReversedIndices) const
{
	return ShouldRenderBackFaces() && !bUseReversedIndices;
}

// FPrimitiveSceneProxy interface.
#if WITH_EDITOR

HHitProxy* UStaticMeshComponent::CreateMeshHitProxy( int32 SectionIndex, int32 MaterialIndex) const
{
	HActor* ActorHitProxy = nullptr; 

	if (GetOwner())
	{	
		ActorHitProxy = new HActor(GetOwner(), this, HitProxyPriority, SectionIndex, MaterialIndex);		
	}

	return ActorHitProxy;
}

HHitProxy* FStaticMeshSceneProxy::CreateHitProxies(UPrimitiveComponent* Component, TArray<TRefCountPtr<HHitProxy> >& OutHitProxies)
{
	// Dispatch to IPrimitiveComponent version
	return FStaticMeshSceneProxy::CreateHitProxies(Component->GetPrimitiveComponentInterface(), OutHitProxies);
}

HHitProxy* FStaticMeshSceneProxy::CreateHitProxies(IPrimitiveComponent* ComponentInterface, TArray<TRefCountPtr<HHitProxy> >& OutHitProxies)
{
	// In order to be able to click on static meshes when they're batched up, we need to have catch all default
	// hit proxy to return.
	HHitProxy* DefaultHitProxy = FPrimitiveSceneProxy::CreateHitProxies(ComponentInterface, OutHitProxies);

	// Sanity check for a case we'll not be handling anymore
	check (! (ComponentInterface->GetUObject<UBrushComponent>()));

	// Generate separate hit proxies for each sub mesh, so that we can perform hit tests against each section for applying materials
	// to each one.
	for ( int32 LODIndex = 0; LODIndex < RenderData->LODResources.Num(); LODIndex++ )
	{
		const FStaticMeshLODResources& LODModel = RenderData->LODResources[LODIndex];

		check(LODs[LODIndex].Sections.Num() == LODModel.Sections.Num());

		for ( int32 SectionIndex = 0; SectionIndex < LODModel.Sections.Num(); SectionIndex++ )
		{
			HHitProxy* HitProxy;

			int32 MaterialIndex = LODModel.Sections[SectionIndex].MaterialIndex;
				
			HitProxy  = ComponentInterface->CreateMeshHitProxy(SectionIndex, MaterialIndex);

			if (HitProxy)
			{
				FLODInfo::FSectionInfo& Section = LODs[LODIndex].Sections[SectionIndex];

				// Set the hitproxy.
				check(Section.HitProxy == NULL);
				Section.HitProxy = HitProxy;

				OutHitProxies.Add(HitProxy);
			}
		}
	}

	return DefaultHitProxy;
}
#endif // WITH_EDITOR

inline void SetupMeshBatchForRuntimeVirtualTexture(FMeshBatch& MeshBatch)
{
	MeshBatch.CastShadow = 0;
	MeshBatch.bUseAsOccluder = 0;
	MeshBatch.bUseForDepthPass = 0;
	MeshBatch.bUseForMaterial = 0;
	MeshBatch.bDitheredLODTransition = 0;
	MeshBatch.bRenderToVirtualTexture = 1;
}

void FStaticMeshSceneProxy::DrawStaticElements(FStaticPrimitiveDrawInterface* PDI)
{
	checkSlow(IsInParallelRenderingThread());
	if (!HasViewDependentDPG())
	{
		// Determine the DPG the primitive should be drawn in.
		ESceneDepthPriorityGroup PrimitiveDPG = GetStaticDepthPriorityGroup();
		int32 NumLODs = RenderData->LODResources.Num();
		//Never use the dynamic path in this path, because only unselected elements will use DrawStaticElements
		bool bIsMeshElementSelected = false;
		const auto FeatureLevel = GetScene().GetFeatureLevel();
		const bool IsMobile = IsMobilePlatform(GetScene().GetShaderPlatform());
		const int32 NumRuntimeVirtualTextureTypes = RuntimeVirtualTextureMaterialTypes.Num();

		//check if a LOD is being forced
		if (ForcedLodModel > 0) 
		{
			int32 LODIndex = FMath::Clamp(ForcedLodModel, ClampedMinLOD + 1, NumLODs) - 1;

			const FStaticMeshLODResources& LODModel = RenderData->LODResources[LODIndex];
			
			// Draw the static mesh elements.
			for(int32 SectionIndex = 0; SectionIndex < LODModel.Sections.Num(); SectionIndex++)
			{
				const FLODInfo::FSectionInfo& Section = LODs[LODIndex].Sections[SectionIndex];
			#if WITH_EDITOR
				if (GIsEditor)
				{
					bIsMeshElementSelected = Section.bSelected;
					PDI->SetHitProxy(Section.HitProxy);
				}
			#endif

				UMaterialInterface* SpecifiedOverlayMaterial = OverlayMaterial;
				if (UMaterialInterface* SectionOverlayMaterial = Section.OverlayMaterial)
				{
					SpecifiedOverlayMaterial = SectionOverlayMaterial;
				}

				const int32 NumBatches = GetNumMeshBatches();
				PDI->ReserveMemoryForMeshes(NumBatches * (1 + NumRuntimeVirtualTextureTypes));

				for (int32 BatchIndex = 0; BatchIndex < NumBatches; BatchIndex++)
				{
					FMeshBatch BaseMeshBatch;

					if (GetMeshElement(LODIndex, BatchIndex, SectionIndex, PrimitiveDPG, bIsMeshElementSelected, true, BaseMeshBatch))
					{
						if (NumRuntimeVirtualTextureTypes > 0)
						{
							// Runtime virtual texture mesh elements.
							FMeshBatch MeshBatch(BaseMeshBatch);
							SetupMeshBatchForRuntimeVirtualTexture(MeshBatch);
							for (ERuntimeVirtualTextureMaterialType MaterialType : RuntimeVirtualTextureMaterialTypes)
							{
								MeshBatch.RuntimeVirtualTextureMaterialType = (uint32)MaterialType;
								PDI->DrawMesh(MeshBatch, FLT_MAX);
							}
						}

						{
							PDI->DrawMesh(BaseMeshBatch, FLT_MAX);
						}

						if (SpecifiedOverlayMaterial != nullptr)
						{
							FMeshBatch OverlayMeshBatch(BaseMeshBatch);
							OverlayMeshBatch.bOverlayMaterial = true;
							OverlayMeshBatch.CastShadow = false;
							OverlayMeshBatch.bSelectable = false;
							OverlayMeshBatch.MaterialRenderProxy = SpecifiedOverlayMaterial->GetRenderProxy();
							// make sure overlay is always rendered on top of base mesh
							OverlayMeshBatch.MeshIdInPrimitive += LODModel.Sections.Num();
							PDI->DrawMesh(OverlayMeshBatch, FLT_MAX);
						}
					}
				}
			}
		} 
		else //no LOD is being forced, submit them all with appropriate cull distances
		{
			for (int32 LODIndex = ClampedMinLOD; LODIndex < NumLODs; ++LODIndex)
			{
				const FStaticMeshLODResources& LODModel = RenderData->LODResources[LODIndex];
				float ScreenSize = GetScreenSize(LODIndex);

				bool bUseUnifiedMeshForShadow = false;
				bool bUseUnifiedMeshForDepth = false;

				if (GUseShadowIndexBuffer && LODModel.bHasDepthOnlyIndices)
				{
					const FLODInfo& ProxyLODInfo = LODs[LODIndex];

					// The shadow-only mesh can be used only if all elements cast shadows and use opaque materials with no vertex modification.
					bool bSafeToUseUnifiedMesh = true;

					bool bAnySectionUsesDitheredLODTransition = false;
					bool bAllSectionsUseDitheredLODTransition = true;
					bool bIsMovable = IsMovable();
					bool bAllSectionsCastShadow = bCastShadow;

					for (int32 SectionIndex = 0; bSafeToUseUnifiedMesh && SectionIndex < LODModel.Sections.Num(); SectionIndex++)
					{
						const FMaterial& Material = ProxyLODInfo.Sections[SectionIndex].Material->GetRenderProxy()->GetIncompleteMaterialWithFallback(FeatureLevel);
						// no support for stateless dithered LOD transitions for movable meshes
						bAnySectionUsesDitheredLODTransition = bAnySectionUsesDitheredLODTransition || (!bIsMovable && Material.IsDitheredLODTransition());
						bAllSectionsUseDitheredLODTransition = bAllSectionsUseDitheredLODTransition && (!bIsMovable && Material.IsDitheredLODTransition());
						const FStaticMeshSection& Section = LODModel.Sections[SectionIndex];

						bSafeToUseUnifiedMesh =
							!(bAnySectionUsesDitheredLODTransition && !bAllSectionsUseDitheredLODTransition) // can't use a single section if they are not homogeneous
							&& Material.WritesEveryPixel()
							&& !Material.IsTwoSided()
							&& !Material.IsThinSurface()
							&& !IsTranslucentBlendMode(Material)
							&& !Material.MaterialModifiesMeshPosition_RenderThread()
							&& Material.GetMaterialDomain() == MD_Surface
							&& !Material.IsSky()
							&& !Material.GetShadingModels().HasShadingModel(MSM_SingleLayerWater);

						bAllSectionsCastShadow &= Section.bCastShadow;
					}

					if (bSafeToUseUnifiedMesh)
					{
						bUseUnifiedMeshForShadow = bAllSectionsCastShadow;

						// Depth pass is only used for deferred renderer. The other conditions are meant to match the logic in FDepthPassMeshProcessor::AddMeshBatch.
						bUseUnifiedMeshForDepth = ShouldUseAsOccluder() && GetScene().GetShadingPath() == EShadingPath::Deferred && !IsMovable();

						if (bUseUnifiedMeshForShadow || bUseUnifiedMeshForDepth)
						{
							const int32 NumBatches = GetNumMeshBatches();

							PDI->ReserveMemoryForMeshes(NumBatches);

							for (int32 BatchIndex = 0; BatchIndex < NumBatches; BatchIndex++)
							{
								FMeshBatch MeshBatch;

								if (GetShadowMeshElement(LODIndex, BatchIndex, PrimitiveDPG, MeshBatch, bAllSectionsUseDitheredLODTransition))
								{
									bUseUnifiedMeshForShadow = bAllSectionsCastShadow;

									MeshBatch.CastShadow = bUseUnifiedMeshForShadow;
									MeshBatch.bUseForDepthPass = bUseUnifiedMeshForDepth;
									MeshBatch.bUseAsOccluder = bUseUnifiedMeshForDepth;
									MeshBatch.bUseForMaterial = false;

									PDI->DrawMesh(MeshBatch, ScreenSize);
								}
							}
						}
					}
				}

				// Draw the static mesh elements.
				for(int32 SectionIndex = 0;SectionIndex < LODModel.Sections.Num();SectionIndex++)
				{
					const FLODInfo::FSectionInfo& Section = LODs[LODIndex].Sections[SectionIndex];
#if WITH_EDITOR
					if( GIsEditor )
					{
						bIsMeshElementSelected = Section.bSelected;
						PDI->SetHitProxy(Section.HitProxy);
					}
#endif // WITH_EDITOR

					UMaterialInterface* SpecifiedOverlayMaterial = OverlayMaterial;
					if (UMaterialInterface* SectionOverlayMaterial = Section.OverlayMaterial)
					{
						SpecifiedOverlayMaterial = SectionOverlayMaterial;
					}

					const int32 NumBatches = GetNumMeshBatches();
					PDI->ReserveMemoryForMeshes(NumBatches * (1 + NumRuntimeVirtualTextureTypes));

					for (int32 BatchIndex = 0; BatchIndex < NumBatches; BatchIndex++)
					{
						FMeshBatch BaseMeshBatch;
						if (GetMeshElement(LODIndex, BatchIndex, SectionIndex, PrimitiveDPG, bIsMeshElementSelected, true, BaseMeshBatch))
						{
							if (NumRuntimeVirtualTextureTypes > 0)
							{
								// Runtime virtual texture mesh elements.
								FMeshBatch MeshBatch(BaseMeshBatch);
								SetupMeshBatchForRuntimeVirtualTexture(MeshBatch);

								for (ERuntimeVirtualTextureMaterialType MaterialType : RuntimeVirtualTextureMaterialTypes)
								{
									MeshBatch.RuntimeVirtualTextureMaterialType = (uint32)MaterialType;
									PDI->DrawMesh(MeshBatch, ScreenSize);
								}
							}

							{
								// Standard mesh elements.
								// If we have submitted an optimized shadow-only mesh, remaining mesh elements must not cast shadows.
								FMeshBatch MeshBatch(BaseMeshBatch);
								MeshBatch.CastShadow &= !bUseUnifiedMeshForShadow;
								MeshBatch.bUseAsOccluder &= !bUseUnifiedMeshForDepth;
								MeshBatch.bUseForDepthPass &= !bUseUnifiedMeshForDepth;
								PDI->DrawMesh(MeshBatch, ScreenSize);
							}

							// negative cull distance disables overlay rendering
							if (SpecifiedOverlayMaterial != nullptr && OverlayMaterialMaxDrawDistance >= 0.f)
							{
								FMeshBatch OverlayMeshBatch(BaseMeshBatch);
								OverlayMeshBatch.bOverlayMaterial = true;
								OverlayMeshBatch.CastShadow = false;
								OverlayMeshBatch.bSelectable = false;
								OverlayMeshBatch.MaterialRenderProxy = SpecifiedOverlayMaterial->GetRenderProxy();
								// make sure overlay is always rendered on top of base mesh
								OverlayMeshBatch.MeshIdInPrimitive += LODModel.Sections.Num();
								// Reuse mesh ScreenSize as cull distance for an overlay. Overlay does not need to compute LOD so we can avoid adding new members into MeshBatch or MeshRelevance
								float OverlayMeshScreenSize = OverlayMaterialMaxDrawDistance;
								PDI->DrawMesh(OverlayMeshBatch, OverlayMeshScreenSize);
							}
						}
					}
				}
			}
		}
	}
}

bool FStaticMeshSceneProxy::IsCollisionView(const FEngineShowFlags& EngineShowFlags, bool& bDrawSimpleCollision, bool& bDrawComplexCollision) const
{
	bDrawSimpleCollision = bDrawComplexCollision = false;

	const bool bInCollisionView = EngineShowFlags.CollisionVisibility || EngineShowFlags.CollisionPawn;

#if STATICMESH_ENABLE_DEBUG_RENDERING
	// If in a 'collision view' and collision is enabled
	if (bInCollisionView && IsCollisionEnabled())
	{
		// See if we have a response to the interested channel
		bool bHasResponse = EngineShowFlags.CollisionPawn && CollisionResponse.GetResponse(ECC_Pawn) != ECR_Ignore;
		bHasResponse |= EngineShowFlags.CollisionVisibility && CollisionResponse.GetResponse(ECC_Visibility) != ECR_Ignore;

		if(bHasResponse)
		{
			// Visibility uses complex and pawn uses simple. However, if UseSimpleAsComplex or UseComplexAsSimple is used we need to adjust accordingly
			bDrawComplexCollision = (EngineShowFlags.CollisionVisibility && CollisionTraceFlag != ECollisionTraceFlag::CTF_UseSimpleAsComplex) || (EngineShowFlags.CollisionPawn && CollisionTraceFlag == ECollisionTraceFlag::CTF_UseComplexAsSimple);
			bDrawSimpleCollision  = (EngineShowFlags.CollisionPawn && CollisionTraceFlag != ECollisionTraceFlag::CTF_UseComplexAsSimple) || (EngineShowFlags.CollisionVisibility && CollisionTraceFlag == ECollisionTraceFlag::CTF_UseSimpleAsComplex);
		}
	}
#endif
	return bInCollisionView;
}

uint8 FStaticMeshSceneProxy::GetCurrentFirstLODIdx_Internal() const
{
	return RenderData->CurrentFirstLODIdx;
}

#if !UE_BUILD_SHIPPING
uint8 FStaticMeshSceneProxy::GetStreamingLODWantedDeficit() const
{
	if (StaticMesh)
	{
		const FStreamableRenderResourceState State = StaticMesh->GetStreamableResourceState();
		const uint8 WantedLODs = StaticMesh->GetCachedNumWantedLODs();
		if (WantedLODs > 0 && State.NumResidentLODs > 0)
		{
			return static_cast<uint8>(FMath::Max(0, (int32)WantedLODs - (int32)State.NumResidentLODs));
		}
	}
	return 0;
}
#endif

void FStaticMeshSceneProxy::OnEvaluateWorldPositionOffsetChanged_RenderThread()
{
	if (ShouldOptimizedWPOAffectNonNaniteShaderSelection())
	{
		// Re-cache draw commands
		GetRendererModule().RequestStaticMeshUpdate(GetPrimitiveSceneInfo());
	}
}

void FStaticMeshSceneProxy::GetMeshDescription(int32 LODIndex, TArray<FMeshBatch>& OutMeshElements) const
{
	const FStaticMeshLODResources& LODModel = RenderData->LODResources[LODIndex];
	const FLODInfo& ProxyLODInfo = LODs[LODIndex];

	for (int32 SectionIndex = 0; SectionIndex < LODModel.Sections.Num(); SectionIndex++)
	{
		const int32 NumBatches = GetNumMeshBatches();

		for (int32 BatchIndex = 0; BatchIndex < NumBatches; BatchIndex++)
		{
			FMeshBatch MeshElement; 

			if (GetMeshElement(LODIndex, BatchIndex, SectionIndex, SDPG_World, false, false, MeshElement))
			{
				OutMeshElements.Add(MeshElement);
			}
		}
	}
}

void FStaticMeshSceneProxy::GetDynamicMeshElements(const TArray<const FSceneView*>& Views, const FSceneViewFamily& ViewFamily, uint32 VisibilityMap, FMeshElementCollector& Collector) const
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_StaticMeshSceneProxy_GetMeshElements);

	const bool bIsLightmapSettingError = HasStaticLighting() && !HasValidSettingsForStaticLighting();
	const bool bProxyIsSelected = IsSelected();
	const FEngineShowFlags& EngineShowFlags = ViewFamily.EngineShowFlags;

	bool bDrawSimpleCollision = false, bDrawComplexCollision = false;
	const bool bInCollisionView = IsCollisionView(EngineShowFlags, bDrawSimpleCollision, bDrawComplexCollision);
	
	// Skip drawing mesh normally if in a collision view, will rely on collision drawing code below
	const bool bDrawMesh = !bInCollisionView && 
	(	IsRichView(ViewFamily) || HasViewDependentDPG()
		|| EngineShowFlags.Collision
#if STATICMESH_ENABLE_DEBUG_RENDERING
		|| bDrawMeshCollisionIfComplex
		|| bDrawMeshCollisionIfSimple
#endif
		|| EngineShowFlags.Bounds
		|| EngineShowFlags.VisualizeInstanceUpdates
		|| bProxyIsSelected 
		|| IsHovered()
		|| bIsLightmapSettingError);

	// Draw polygon mesh if we are either not in a collision view, or are drawing it as collision.
	if (EngineShowFlags.StaticMeshes && bDrawMesh)
	{
		// how we should draw the collision for this mesh.
		const bool bIsWireframeView = EngineShowFlags.Wireframe;
		const bool bActorColorationEnabled = EngineShowFlags.ActorColoration;
		const ERHIFeatureLevel::Type FeatureLevel = ViewFamily.GetFeatureLevel();

		for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ViewIndex++)
		{
			const FSceneView* View = Views[ViewIndex];
			const bool bLODColoration = View->Family->EngineShowFlags.LODColoration;

			if (IsShown(View) && (VisibilityMap & (1 << ViewIndex)))
			{
				FLODMask LODMask = GetLODMask(View);

				for (int32 LODIndex = 0; LODIndex < RenderData->LODResources.Num(); LODIndex++)
				{
					if (LODMask.ContainsLOD(LODIndex) && LODIndex >= ClampedMinLOD)
					{
						const FStaticMeshLODResources& LODModel = RenderData->LODResources[LODIndex];
						const FLODInfo& ProxyLODInfo = LODs[LODIndex];

						if (AllowDebugViewmodes() && bIsWireframeView && !EngineShowFlags.Materials
							// If any of the materials are mesh-modifying, we can't use the single merged mesh element of GetWireframeMeshElement()
							&& !ProxyLODInfo.UsesMeshModifyingMaterials())
						{
							const FLinearColor ViewWireframeColor( bActorColorationEnabled ? GetPrimitiveColor() : GetWireframeColor() );

							auto WireframeMaterialInstance = new FColoredMaterialRenderProxy(
								GEngine->WireframeMaterial->GetRenderProxy(),
								GetSelectionColor(ViewWireframeColor,!(GIsEditor && EngineShowFlags.Selection) || bProxyIsSelected, IsHovered(), false)
								);

							Collector.RegisterOneFrameMaterialProxy(WireframeMaterialInstance);

							const int32 NumBatches = GetNumMeshBatches();

							for (int32 BatchIndex = 0; BatchIndex < NumBatches; BatchIndex++)
							{
								// GetWireframeMeshElement will try SetIndexSource at section index 0
								// and GetMeshElement loops over sections, therefore does not have this issue
								if (LODModel.Sections.Num() > 0)
								{
									FMeshBatch& Mesh = Collector.AllocateMesh();

									if (GetWireframeMeshElement(LODIndex, BatchIndex, WireframeMaterialInstance, SDPG_World, true, Mesh))
									{
										// We implemented our own wireframe
										Mesh.bCanApplyViewModeOverrides = false;
										Collector.AddMesh(ViewIndex, Mesh);
										INC_DWORD_STAT_BY(STAT_StaticMeshTriangles, Mesh.GetNumPrimitives());
									}
								}
							}
						}
						else
						{
							const FLinearColor UtilColor( GetPrimitiveColor() );

							// Draw the static mesh sections.
							for (int32 SectionIndex = 0; SectionIndex < LODModel.Sections.Num(); SectionIndex++)
							{
								const int32 NumBatches = GetNumMeshBatches();

								for (int32 BatchIndex = 0; BatchIndex < NumBatches; BatchIndex++)
								{
									bool bSectionIsSelected = false;
									FMeshBatch& MeshElement = Collector.AllocateMesh();

	#if WITH_EDITOR
									if (GIsEditor)
									{
										if (EngineShowFlags.FrontBackFace)
										{
											MeshElement.bDisableBackfaceCulling = true;
										}

										const FLODInfo::FSectionInfo& Section = LODs[LODIndex].Sections[SectionIndex];

										bSectionIsSelected = Section.bSelected || (bIsWireframeView && bProxyIsSelected);
										MeshElement.BatchHitProxyId = Section.HitProxy ? Section.HitProxy->Id : FHitProxyId();
									}
								#endif
								
									if (GetMeshElement(LODIndex, BatchIndex, SectionIndex, SDPG_World, bSectionIsSelected, true, MeshElement))
									{
										bool bDebugMaterialRenderProxySet = false;
#if STATICMESH_ENABLE_DEBUG_RENDERING

	#if WITH_EDITOR								
										if (bProxyIsSelected && EngineShowFlags.PhysicalMaterialMasks && AllowDebugViewmodes())
										{
											// Override the mesh's material with our material that draws the physical material masks
											UMaterial* PhysMatMaskVisualizationMaterial = GEngine->PhysicalMaterialMaskMaterial;
											check(PhysMatMaskVisualizationMaterial);
											
											FMaterialRenderProxy* PhysMatMaskVisualizationMaterialInstance = nullptr;

											const FLODInfo::FSectionInfo& Section = LODs[LODIndex].Sections[SectionIndex];
											
											if (UMaterialInterface* SectionMaterial = Section.Material)
											{
												if (UPhysicalMaterialMask* PhysicalMaterialMask = SectionMaterial->GetPhysicalMaterialMask())
												{
													if (PhysicalMaterialMask->MaskTexture)
													{
														PhysMatMaskVisualizationMaterialInstance = new FColoredTexturedMaterialRenderProxy(
															PhysMatMaskVisualizationMaterial->GetRenderProxy(),
															FLinearColor::White, NAME_Color, PhysicalMaterialMask->MaskTexture, NAME_LinearColor);

														Collector.RegisterOneFrameMaterialProxy(PhysMatMaskVisualizationMaterialInstance);
														MeshElement.MaterialRenderProxy = PhysMatMaskVisualizationMaterialInstance;

														bDebugMaterialRenderProxySet = true;
													}
												}
											}
										}

										// Only draw the selection outline for selected items which only write to runtime virtual texture.
										if (bProxyIsSelected && DrawInRuntimeVirtualTextureOnly(ViewFamily.EngineShowFlags.Editor))
										{
											if (FColoredMaterialRenderProxy* DebugMaterialProxy = new FColoredMaterialRenderProxy(GEngine->EmissiveMeshMaterial->GetRenderProxy(), FColor::Transparent))
											{
												Collector.RegisterOneFrameMaterialProxy(DebugMaterialProxy);
												MeshElement.MaterialRenderProxy = DebugMaterialProxy;
												bDebugMaterialRenderProxySet = true;
											}
										}
	#endif // WITH_EDITOR
										// Override the mesh's material with our material that draws vertex color
										if (!bDebugMaterialRenderProxySet && bProxyIsSelected && EngineShowFlags.VertexColors && AllowDebugViewmodes())
										{
											if (FMaterialRenderProxy* VertexColorVisualizationMaterialInstance = MeshPaintVisualize::GetMaterialRenderProxy(bSectionIsSelected, IsHovered()))
											{
												Collector.RegisterOneFrameMaterialProxy(VertexColorVisualizationMaterialInstance);
												MeshElement.MaterialRenderProxy = VertexColorVisualizationMaterialInstance;
												bDebugMaterialRenderProxySet = true;
											}
										}

	#endif // STATICMESH_ENABLE_DEBUG_RENDERING
	#if WITH_EDITOR
										if (!bDebugMaterialRenderProxySet && bSectionIsSelected)
										{
											// Override the mesh's material with our material that draws the collision color
											MeshElement.MaterialRenderProxy = new FOverrideSelectionColorMaterialRenderProxy(
												GEngine->ShadedLevelColorationUnlitMaterial->GetRenderProxy(),
												GetSelectionColor(GEngine->GetSelectedMaterialColor(), bSectionIsSelected, IsHovered())
											);
										}
	#endif
										if (MeshElement.bDitheredLODTransition && LODMask.IsDithered())
										{
#if UE_ENABLE_DEBUG_DRAWING
											if(bLODColoration)
											{ 
												if (LODIndex == LODMask.LODIndex0)
												{
													MeshElement.CachedLODDither = FMeshBatch::EDitherState::FadeOut;
												}
												else if (LODIndex == LODMask.LODIndex1)
												{
													MeshElement.CachedLODDither = FMeshBatch::EDitherState::FadeIn;
												}
											}
#endif										
										}
										else
										{
											MeshElement.bDitheredLODTransition = false;
										}
								
										MeshElement.bCanApplyViewModeOverrides = true;
										MeshElement.bUseWireframeSelectionColoring = bSectionIsSelected;

										Collector.AddMesh(ViewIndex, MeshElement);
										INC_DWORD_STAT_BY(STAT_StaticMeshTriangles,MeshElement.GetNumPrimitives());
									}
								}
							}
						}
					}
				}
			}
		}
	}
	
#if STATICMESH_ENABLE_DEBUG_RENDERING
	// Collision and bounds drawing
	FColor SimpleCollisionColor = FColor(157, 149, 223, 255);
	FColor ComplexCollisionColor = FColor(0, 255, 255, 255);

	for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ViewIndex++)
	{
		if (VisibilityMap & (1 << ViewIndex))
		{
			if(AllowDebugViewmodes())
			{
				// Should we draw the mesh wireframe to indicate we are using the mesh as collision
				bool bDrawComplexWireframeCollision = (EngineShowFlags.Collision && IsCollisionEnabled() && CollisionTraceFlag == ECollisionTraceFlag::CTF_UseComplexAsSimple);
				// Requested drawing complex in wireframe, but check that we are not using simple as complex
				bDrawComplexWireframeCollision |= (bDrawMeshCollisionIfComplex && CollisionTraceFlag != ECollisionTraceFlag::CTF_UseSimpleAsComplex);
				// Requested drawing simple in wireframe, and we are using complex as simple
				bDrawComplexWireframeCollision |= (bDrawMeshCollisionIfSimple && CollisionTraceFlag == ECollisionTraceFlag::CTF_UseComplexAsSimple);

				// If drawing complex collision as solid or wireframe
				if(bDrawComplexWireframeCollision || (bInCollisionView && bDrawComplexCollision))
				{
					// If we have at least one valid LOD to draw
					if(RenderData->LODResources.Num() > 0)
					{
						// Get LOD used for collision
						int32 DrawLOD = FMath::Clamp(LODForCollision, 0, RenderData->LODResources.Num() - 1);
						const FStaticMeshLODResources& LODModel = RenderData->LODResources[DrawLOD];

						UMaterial* MaterialToUse = UMaterial::GetDefaultMaterial(MD_Surface);
						FLinearColor DrawCollisionColor = GetWireframeColor();
						// Collision view modes draw collision mesh as solid
						if(bInCollisionView)
						{
							MaterialToUse = GEngine->ShadedLevelColorationUnlitMaterial;
						}
						// Wireframe, choose color based on complex or simple
						else
						{
							MaterialToUse = GEngine->WireframeMaterial;
							DrawCollisionColor = (CollisionTraceFlag == ECollisionTraceFlag::CTF_UseComplexAsSimple) ? SimpleCollisionColor : ComplexCollisionColor;
						}

						// Iterate over sections of that LOD
						for (int32 SectionIndex = 0; SectionIndex < LODModel.Sections.Num(); SectionIndex++)
						{
							// If this section has collision enabled
							if (LODModel.Sections[SectionIndex].bEnableCollision)
							{
#if WITH_EDITOR
								// See if we are selected
								const bool bSectionIsSelected = LODs[DrawLOD].Sections[SectionIndex].bSelected;
#else
								const bool bSectionIsSelected = false;
#endif

								// Create colored proxy
								FColoredMaterialRenderProxy* CollisionMaterialInstance = new FColoredMaterialRenderProxy(MaterialToUse->GetRenderProxy(), DrawCollisionColor);
								Collector.RegisterOneFrameMaterialProxy(CollisionMaterialInstance);

								// Iterate over batches
								for (int32 BatchIndex = 0; BatchIndex < GetNumMeshBatches(); BatchIndex++)
								{
									FMeshBatch& CollisionElement = Collector.AllocateMesh();
									if (GetCollisionMeshElement(DrawLOD, BatchIndex, SectionIndex, SDPG_World, CollisionMaterialInstance, CollisionElement))
									{
										Collector.AddMesh(ViewIndex, CollisionElement);
										INC_DWORD_STAT_BY(STAT_StaticMeshTriangles, CollisionElement.GetNumPrimitives());
									}
								}
							}
						}
					}
				}
			}

			// Draw simple collision as wireframe if 'show collision', collision is enabled, and we are not using the complex as the simple
			const bool bDrawSimpleWireframeCollision = (EngineShowFlags.Collision && IsCollisionEnabled() && CollisionTraceFlag != ECollisionTraceFlag::CTF_UseComplexAsSimple);

			if((bDrawSimpleCollision || bDrawSimpleWireframeCollision) && BodySetup)
			{
				if(FMath::Abs(GetLocalToWorld().Determinant()) < UE_SMALL_NUMBER)
				{
					// Catch this here or otherwise GeomTransform below will assert
					// This spams so commented out
					//UE_LOGF(LogStaticMesh, Log, "Zero scaling not supported (%ls)", *StaticMesh->GetPathName());
				}
				else
				{
					const bool bDrawSolid = !bDrawSimpleWireframeCollision;

					if(AllowDebugViewmodes() && bDrawSolid)
					{
						// Make a material for drawing solid collision stuff
						auto SolidMaterialInstance = new FColoredMaterialRenderProxy(
							GEngine->ShadedLevelColorationUnlitMaterial->GetRenderProxy(),
							GetWireframeColor()
							);

						Collector.RegisterOneFrameMaterialProxy(SolidMaterialInstance);

						FTransform GeomTransform(GetLocalToWorld());
						BodySetup->AggGeom.GetAggGeom(GeomTransform, GetWireframeColor().ToFColor(true), SolidMaterialInstance, false, true, AlwaysHasVelocity(), ViewIndex, Collector);
					}
					// wireframe
					else
					{
						FTransform GeomTransform(GetLocalToWorld());
						BodySetup->AggGeom.GetAggGeom(GeomTransform, GetSelectionColor(SimpleCollisionColor, bProxyIsSelected, IsHovered()).ToFColor(true), NULL, ( Owner == NULL ), false, AlwaysHasVelocity(), ViewIndex, Collector);
					}


					// The simple nav geometry is only used by dynamic obstacles for now
					if (StaticMesh->GetNavCollision() && StaticMesh->GetNavCollision()->IsDynamicObstacle())
					{
						// Draw the static mesh's body setup (simple collision)
						FTransform GeomTransform(GetLocalToWorld());
						FColor NavCollisionColor = FColor(118,84,255,255);
						StaticMesh->GetNavCollision()->DrawSimpleGeom(Collector.GetPDI(ViewIndex), GeomTransform, GetSelectionColor(NavCollisionColor, bProxyIsSelected, IsHovered()).ToFColor(true));
					}
				}
			}

			if(EngineShowFlags.MassProperties && DebugMassData.Num() > 0)
			{
				DebugMassData[0].DrawDebugMass(Collector.GetPDI(ViewIndex), FTransform(GetLocalToWorld()));
			}
	
			if (EngineShowFlags.StaticMeshes)
			{
				RenderBounds(Collector.GetPDI(ViewIndex), EngineShowFlags, GetBounds(), !Owner || IsSelected());
			}
		}
	}
#endif // STATICMESH_ENABLE_DEBUG_RENDERING
}


const FCardRepresentationData* FStaticMeshSceneProxy::GetMeshCardRepresentation() const
{
	return CardRepresentationData;
}


#if RHI_RAYTRACING
bool FStaticMeshSceneProxy::HasRayTracingRepresentation() const
{
	return bHasRayTracingRepresentation;
}

TArray<FRayTracingGeometry*> FStaticMeshSceneProxy::GetStaticRayTracingGeometries() const
{
	if (bSupportRayTracing)
	{
		FStaticMeshRayTracingProxyLODArray& RayTracingLODs = RenderData->RayTracingProxy->LODs;

		TArray<FRayTracingGeometry*> RayTracingGeometries;
		RayTracingGeometries.AddDefaulted(RayTracingLODs.Num());
		for (int32 LODIndex = 0; LODIndex < RayTracingLODs.Num(); LODIndex++)
		{
			RayTracingGeometries[LODIndex] = RayTracingLODs[LODIndex].RayTracingGeometry;
		}

		return MoveTemp(RayTracingGeometries);
	}

	return {};
}

void FStaticMeshSceneProxy::GetStaticRayTracingInstances(FStaticRayTracingInstances& OutRayTracingInstances)
{
	const int32 NumBatches = GetNumMeshBatches();
	const ESceneDepthPriorityGroup PrimitiveDPG = GetStaticDepthPriorityGroup();

	const FStaticMeshRayTracingProxyLODArray& RayTracingLODs = RenderData->RayTracingProxy->LODs;
	const int32 NumLODs = RayTracingLODs.Num();
	const int32 RayTracingMinLOD = RenderData->RayTracingProxy->bUsingRenderingLODs ? ClampedMinLOD : 0;

	OutRayTracingInstances.LODs.SetNum(NumLODs);

	for (int32 LODIndex = RayTracingMinLOD; LODIndex < NumLODs; ++LODIndex)
	{
		const FStaticMeshRayTracingProxyLOD& RayTracingLOD = RayTracingLODs[LODIndex];

		TArray<FMeshBatch> RayTracingMaterials;

		for (int32 BatchIndex = 0; BatchIndex < NumBatches; BatchIndex++)
		{
			for (int32 SectionIndex = 0; SectionIndex < RayTracingLOD.Sections->Num(); SectionIndex++)
			{
				FMeshBatch MeshBatch;

				bool bResult;
				if (RenderData->RayTracingProxy->bUsingRenderingLODs)
				{
					// when using rendering LODs we can reuse the main GetMeshElement(...)
					bResult = GetMeshElement(LODIndex, BatchIndex, SectionIndex, PrimitiveDPG, false, false, MeshBatch);
				}
				else
				{
					// otherwise initialize MeshBatch using ray tracing proxy VB/IB/Section data
					bResult = GetRayTracingMeshElement(LODIndex, BatchIndex, SectionIndex, PrimitiveDPG, MeshBatch);
				}

				if (bResult)
				{
					MeshBatch.ReverseCulling = bReverseCulling; // overwrite what came from GetMeshElement as DXR only needs the user driven flag, not the flipping implied by the transform
					MeshBatch.SegmentIndex = SectionIndex;
					MeshBatch.MeshIdInPrimitive = SectionIndex;

					RayTracingMaterials.Add(MeshBatch);
				}
			}
		}

		if (RayTracingMaterials.IsEmpty())
		{
			continue;
		}

		FRayTracingInstance& OutRayTracingInstance = OutRayTracingInstances.LODs[LODIndex].Instances.AddDefaulted_GetRef();
		OutRayTracingInstance.Geometry = RayTracingLOD.RayTracingGeometry;
		OutRayTracingInstance.Materials = MoveTemp(RayTracingMaterials);
	}
}

RayTracing::FGeometryGroupHandle FStaticMeshSceneProxy::GetRayTracingGeometryGroupHandle() const
{
	check(IsInRenderingThread() || IsInParallelRenderingThread());
	return RayTracingGeometryGroupHandle;
}

FRaytracingVFAttributeData FStaticMeshSceneProxy::GetRaytracingVFAttributeData(const FSceneView* View)
{
	FRaytracingVFAttributeData Out;

	if (!bSupportRayTracing || !bHasRayTracingRepresentation || !IsRayTracingRelevant())
	{
		return Out;
	}

	if (!RenderData || !RenderData->RayTracingProxy || !RenderData->RayTracingProxy->LODVertexFactories)
	{
		return Out;
	}

	const int32 LODIndex = (View && RenderData->RayTracingProxy->bUsingRenderingLODs) ? FMath::Max(GetLOD(View), (int32)GetCurrentFirstLODIdx_RenderThread()) : 0;
	const FStaticMeshVertexFactories& VertexFactories = (*RenderData->RayTracingProxy->LODVertexFactories)[LODIndex];
	FRaytracingVFSectionData& Section = Out.Sections.Emplace_GetRef();
	Section.UV.BufferSRV = VertexFactories.VertexFactory.GetTextureCoordinatesSRV();
	Section.UV.NumCoordinates = VertexFactories.VertexFactory.GetNumTexcoords();

	return Out;
}

void FStaticMeshSceneProxy::GetDynamicRayTracingInstances(FRayTracingInstanceCollector& Collector)
{
#if DO_CHECK
	// TODO: Once workaround below is removed we should check bDynamicRayTracingGeometry here 
	if (!ensureMsgf(IsRayTracingRelevant() && bSupportRayTracing && bHasRayTracingRepresentation,
		TEXT("FStaticMeshSceneProxy::GetDynamicRayTracingInstances(...) should only be called for proxies using dynamic raytracing geometry. ")
		TEXT("Ray tracing primitive gathering code may be wrong.")))
	{
		return;
	}
#endif

	// Workaround: SetEvaluateWorldPositionOffsetInRayTracing(...) calls UpdateCachedRayTracingState(...)
	// however the update only happens after gathering relevant ray tracing primitives
	// so ERayTracingPrimitiveFlags::Dynamic is set for one frame after the WPO evaluation is disabled.
	if (!bDynamicRayTracingGeometry)
	{
		return;
	}

	if (CVarRayTracingStaticMeshes.GetValueOnRenderThread() == 0)
	{
		// TODO: Exclude proxy during ray tracing primitive gather instead of doing this early out here.
		return;
	}

	checkf(!DynamicRayTracingGeometries.IsEmpty(), TEXT("Proxy should have entries in DynamicRayTracingGeometries when using the GetDynamicRayTracingInstances() code path."));

	TConstArrayView<const FSceneView*> Views = Collector.GetViews();
	const uint32 VisibilityMap = Collector.GetVisibilityMap();

	// RT geometry will be generated based on first active view and then reused for all other views
	// TODO: Expose a way for developers to control whether to reuse RT geometry or create one per-view
	const int32 FirstActiveViewIndex = FMath::CountTrailingZeros(VisibilityMap);
	checkf(Views.IsValidIndex(FirstActiveViewIndex), TEXT("There should be at least one active view when calling GetDynamicRayTracingInstances(...)."));

	const ESceneDepthPriorityGroup PrimitiveDPG = GetStaticDepthPriorityGroup();

	FStaticMeshRayTracingProxyLODArray& RayTracingLODs = RenderData->RayTracingProxy->LODs;
	const int32 NumLODs = RayTracingLODs.Num();

	bool bEvaluateWPO = false;

	int32 RayTracingMinLOD = RenderData->RayTracingProxy->bUsingRenderingLODs ? NumLODs - 1 : 0;

	for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ViewIndex++)
	{
		if ((VisibilityMap & (1 << ViewIndex)) == 0)
		{
			continue;
		}

		const FSceneView* View = Views[ViewIndex];

		const FVector ViewCenter = View->ViewMatrices.GetViewOrigin();
		bEvaluateWPO |= ShouldEvaluateWPOInRayTracing(ViewCenter);

		if (RenderData->RayTracingProxy->bUsingRenderingLODs)
		{
			// we use same LOD for all views
			RayTracingMinLOD = FMath::Min(RayTracingMinLOD, GetLOD(View));
		}
	}

	if (RenderData->RayTracingProxy->bUsingRenderingLODs)
	{
		// when using rendering LODs also need to limit based on currently loaded LOD
		RayTracingMinLOD = FMath::Max(RayTracingMinLOD, (int32)GetCurrentFirstLODIdx_RenderThread());
	}

	int32 LODIndex = RayTracingMinLOD;

	if (bEvaluateWPO && !RenderData->RayTracingProxy->bUsingRenderingLODs)
	{
		// when using WPO, need to mark the geometry group as referenced since VB/IB need to be streamed-in 
		// TODO: Support streaming only buffers when using dynamic geometry
		Collector.AddReferencedGeometryGroupForDynamicUpdate(RenderData->RayTracingGeometryGroupHandle);

		// select first LOD with valid VB/IB
		for (; LODIndex < NumLODs; LODIndex++)
		{
			const FStaticMeshRayTracingProxyLOD& CurrentRayTracingLOD = RayTracingLODs[LODIndex];

			if (CurrentRayTracingLOD.bBuffersInlined || CurrentRayTracingLOD.AreBuffersStreamedIn())
			{
				break;
			}
		}

		if (LODIndex == INDEX_NONE || LODIndex >= NumLODs)
		{
			// if none of the LODs have buffers ready for dynamic BLAS update, fallback to static BLAS
			bEvaluateWPO = false;

			LODIndex = RayTracingMinLOD;
		}
	}

	if (!bEvaluateWPO)
	{
		// when not using WPO, need to mark the geometry group as referenced (for streaming/residency management)
		// since the static ray tracing geometry will be used

		Collector.AddReferencedGeometryGroup(RenderData->RayTracingGeometryGroupHandle);

		// select first LOD with valid ray tracing geometry
		for (; LODIndex < NumLODs; LODIndex++)
		{
			FStaticMeshRayTracingProxyLOD& CurrentRayTracingLOD = RayTracingLODs[LODIndex];

			if (CurrentRayTracingLOD.RayTracingGeometry->HasPendingBuildRequest())
			{
				CurrentRayTracingLOD.RayTracingGeometry->BoostBuildPriority();
			}
			else if (CurrentRayTracingLOD.RayTracingGeometry->IsValid() && !CurrentRayTracingLOD.RayTracingGeometry->IsEvicted())
			{
				break;
			}
		}
	}

	if (LODIndex == INDEX_NONE || LODIndex >= NumLODs)
	{
		return;
	}

	const FStaticMeshRayTracingProxyLOD& RayTracingLOD = RayTracingLODs[LODIndex];

	if (RayTracingLOD.VertexBuffers->StaticMeshVertexBuffer.GetNumVertices() <= 0)
	{
		return;
	}

	// TODO: Need to validate that the DynamicRayTracingGeometries are still valid - they could contain streamed out IndexBuffers from the shared StaticMesh (UE-139474)
	FRayTracingGeometry& Geometry = bEvaluateWPO ? DynamicRayTracingGeometries[LODIndex] : *RayTracingLOD.RayTracingGeometry;

	if (bEvaluateWPO)
	{
		if (Geometry.HasPendingBuildRequest())
		{
			// This should only happen if geometry was recently made resident and build hasn't happened yet.
			// TODO: could cancel build request and let it go through the dynamic code path which will build it as necessary
			return;
		}
	}
	else
	{
		check(Geometry.IsValid() && !Geometry.IsEvicted() && !Geometry.HasPendingBuildRequest());
	}

	{
		FRayTracingInstance RayTracingInstance;
	
		const int32 NumBatches = GetNumMeshBatches();
		const int32 NumRayTracingMaterialEntries = RayTracingLOD.Sections->Num() * NumBatches;

		if (NumRayTracingMaterialEntries != CachedRayTracingMaterials.Num() || CachedRayTracingMaterialsLODIndex != LODIndex)
		{
			CachedRayTracingMaterials.Reset();
			CachedRayTracingMaterials.Reserve(NumRayTracingMaterialEntries);

			for (int32 BatchIndex = 0; BatchIndex < NumBatches; BatchIndex++)
			{
				for (int32 SectionIndex = 0; SectionIndex < RayTracingLOD.Sections->Num(); SectionIndex++)
				{
					FMeshBatch MeshBatch;

					bool bResult;
					if (RenderData->RayTracingProxy->bUsingRenderingLODs)
					{
						// when using rendering LODs we can reuse the main GetMeshElement(...)
						bResult = GetMeshElement(LODIndex, BatchIndex, SectionIndex, PrimitiveDPG, false, false, MeshBatch);
					}
					else
					{
						// otherwise initialize MeshBatch using ray tracing proxy VB/IB/Section data
						bResult = GetRayTracingMeshElement(LODIndex, BatchIndex, SectionIndex, PrimitiveDPG, MeshBatch);
					}

					if (bResult)
					{
						MeshBatch.ReverseCulling = bReverseCulling; // overwrite what came from GetMeshElement as DXR only needs the user driven flag, not the flipping implied by the transform
						MeshBatch.SegmentIndex = SectionIndex;
						MeshBatch.MeshIdInPrimitive = SectionIndex;

						CachedRayTracingMaterials.Add(MoveTemp(MeshBatch));
					}
				}
			}
			
			RayTracingInstance.MaterialsView = MakeArrayView(CachedRayTracingMaterials);
			CachedRayTracingMaterialsLODIndex = LODIndex;
		}
		else
		{
			RayTracingInstance.MaterialsView = MakeArrayView(CachedRayTracingMaterials);

			// Skip computing the mask and flags in the renderer since we are using cached values.
			RayTracingInstance.bInstanceMaskAndFlagsDirty = false;
		}

		RayTracingInstance.Geometry = &Geometry;

		// scene proxies live for the duration of Render(), making array views below safe
		const FMatrix& ThisLocalToWorld = GetLocalToWorld();
		RayTracingInstance.InstanceTransformsView = MakeArrayView(&ThisLocalToWorld, 1);

		if (bEvaluateWPO && (*RenderData->RayTracingProxy->LODVertexFactories)[LODIndex].VertexFactory.GetType()->SupportsRayTracingDynamicGeometry())
		{
			const uint32 NumVertices = RayTracingLOD.VertexBuffers->PositionVertexBuffer.GetNumVertices();

			FRayTracingDynamicGeometryUpdateParams UpdateParams;
			UpdateParams.MeshBatchesView = CachedRayTracingMaterials;
			UpdateParams.NumVertices = NumVertices;
			UpdateParams.VertexBufferSize = uint32((size_t)NumVertices * sizeof(FVector3f));
			UpdateParams.NumTriangles = Geometry.GetInitializer().TotalPrimitiveCount;
			UpdateParams.Geometry = &Geometry;
			UpdateParams.Buffer = nullptr; // Use the shared vertex buffer - needs to be updated every frame
			Collector.AddRayTracingGeometryUpdate(FirstActiveViewIndex, MoveTemp(UpdateParams));
		}

		check(CachedRayTracingMaterials.Num() == RayTracingInstance.GetMaterials().Num());
		checkf(RayTracingInstance.Geometry->GetInitializer().Segments.Num() >= CachedRayTracingMaterials.Num(), TEXT("Segments/Materials mismatch. Number of segments: %d. Number of Materials: %d. LOD Index: %d"),
			RayTracingInstance.Geometry->GetInitializer().Segments.Num(),
			CachedRayTracingMaterials.Num(), 
			LODIndex);

		for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ViewIndex++)
		{
			if ((VisibilityMap & (1 << ViewIndex)) == 0)
			{
				continue;
			}

			Collector.AddRayTracingInstance(ViewIndex, RayTracingInstance);
		}
	}
}
#endif

void FStaticMeshSceneProxy::GetLCIs(FLCIArray& LCIs)
{
	for (int32 LODIndex = 0; LODIndex < LODs.Num(); ++LODIndex)
	{
		FLightCacheInterface* LCI = &LODs[LODIndex];
		LCIs.Push(LCI);
	}
}

bool FStaticMeshSceneProxy::CanBeOccluded() const
{
	return !MaterialRelevance.bDisableDepthTest && !MaterialRelevance.bPostMotionBlurTranslucency && !ShouldRenderCustomDepth() && !IsRuntimeVirtualTextureOnly();
}

bool FStaticMeshSceneProxy::IsUsingDistanceCullFade() const
{
	return MaterialRelevance.bUsesDistanceCullFade;
}


FPrimitiveViewRelevance FStaticMeshSceneProxy::GetViewRelevance(const FSceneView* View) const
{   
	checkSlow(IsInParallelRenderingThread());

	FPrimitiveViewRelevance Result;
	Result.bDrawRelevance = IsShown(View) && View->Family->EngineShowFlags.StaticMeshes;
	Result.bRenderCustomDepth = ShouldRenderCustomDepth();
	Result.bRenderInMainPass = ShouldRenderInMainPass();
	Result.bRenderInDepthPass = ShouldRenderInDepthPass();
	Result.bUsesLightingChannels = GetLightingChannelMask() != GetDefaultLightingChannelMask();
	Result.bTranslucentSelfShadow = bCastVolumetricTranslucentShadow;

#if STATICMESH_ENABLE_DEBUG_RENDERING
	bool bDrawSimpleCollision = false, bDrawComplexCollision = false;
	const bool bInCollisionView = IsCollisionView(View->Family->EngineShowFlags, bDrawSimpleCollision, bDrawComplexCollision);
#else
	bool bInCollisionView = false;
#endif
	const bool bAllowStaticLighting = IsStaticLightingAllowed();

	if(
#if !(UE_BUILD_SHIPPING) || WITH_EDITOR
		IsRichView(*View->Family) || 
		View->Family->EngineShowFlags.Collision ||
		bInCollisionView ||
		View->Family->EngineShowFlags.Bounds ||
		View->Family->EngineShowFlags.VisualizeInstanceUpdates ||
#endif
#if WITH_EDITOR
		(IsSelected() && View->Family->EngineShowFlags.VertexColors) ||
		(IsSelected() && View->Family->EngineShowFlags.PhysicalMaterialMasks) ||
		(IsSelected() && DrawInRuntimeVirtualTextureOnly(View->Family->EngineShowFlags.Editor)) ||
#endif
#if STATICMESH_ENABLE_DEBUG_RENDERING
		bDrawMeshCollisionIfComplex ||
		bDrawMeshCollisionIfSimple ||
#endif
		// Force down dynamic rendering path if invalid lightmap settings, so we can apply an error material in DrawRichMesh
		(bAllowStaticLighting && HasStaticLighting() && !HasValidSettingsForStaticLighting()) ||
		HasViewDependentDPG()
		)
	{
		Result.bDynamicRelevance = true;

#if STATICMESH_ENABLE_DEBUG_RENDERING
		// If we want to draw collision, needs to make sure we are considered relevant even if hidden
		if(View->Family->EngineShowFlags.Collision || bInCollisionView)
		{
			Result.bDrawRelevance = true;
		}
#endif
	}
	else
	{
		Result.bStaticRelevance = true;

#if WITH_EDITOR
		//only check these in the editor
		Result.bEditorVisualizeLevelInstanceRelevance = IsEditingLevelInstanceChild();
		Result.bEditorStaticSelectionRelevance = (WantsEditorEffects() || IsSelected() || IsHovered());
#endif
	}

	Result.bShadowRelevance = IsShadowCast(View);

	MaterialRelevance.SetPrimitiveViewRelevance(Result);

	if (!View->Family->EngineShowFlags.Materials 
#if STATICMESH_ENABLE_DEBUG_RENDERING
		|| bInCollisionView
#endif
		)
	{
		Result.bOpaque = true;
	}

	Result.bVelocityRelevance = (VelocityIncludeStationaryPrimitives(View->GetShaderPlatform()) || DrawsVelocity()) && Result.bOpaque && Result.bRenderInMainPass;

	return Result;
}

void FStaticMeshSceneProxy::GetLightRelevance(const FLightSceneProxy* LightSceneProxy, bool& bDynamic, bool& bRelevant, bool& bLightMapped, bool& bShadowMapped) const
{
	// Attach the light to the primitive's static meshes.
	bDynamic = true;
	bRelevant = false;
	bLightMapped = true;
	bShadowMapped = true;

	if (LODs.Num() > 0)
	{
		for (int32 LODIndex = 0; LODIndex < LODs.Num(); LODIndex++)
		{
			const FLODInfo& LCI = LODs[LODIndex];

			ELightInteractionType InteractionType = LCI.GetInteraction(LightSceneProxy).GetType();

			if (InteractionType != LIT_CachedIrrelevant)
			{
				bRelevant = true;
			}

			if (InteractionType != LIT_CachedLightMap && InteractionType != LIT_CachedIrrelevant)
			{
				bLightMapped = false;
			}

			if (InteractionType != LIT_Dynamic)
			{
				bDynamic = false;
			}

			if (InteractionType != LIT_CachedSignedDistanceFieldShadowMap2D)
			{
				bShadowMapped = false;
			}
		}
	}
	else
	{
		bRelevant = true;
		bLightMapped = false;
		bShadowMapped = false;
	}
}

void FStaticMeshSceneProxy::GetDistanceFieldAtlasData(const FDistanceFieldVolumeData*& OutDistanceFieldData, float& SelfShadowBias) const
{
	OutDistanceFieldData = DistanceFieldData;
	SelfShadowBias = DistanceFieldSelfShadowBias;
}

bool FStaticMeshSceneProxy::HasDistanceFieldRepresentation() const
{
	return CastsDynamicShadow() && AffectsDistanceFieldLighting() && DistanceFieldData;
}

bool FStaticMeshSceneProxy::StaticMeshHasPendingStreaming() const
{
	return StaticMesh && StaticMesh->bHasStreamingUpdatePending;
}

bool FStaticMeshSceneProxy::HasDynamicIndirectShadowCasterRepresentation() const
{
	return bCastsDynamicIndirectShadow && FStaticMeshSceneProxy::HasDistanceFieldRepresentation();
}

/** Initialization constructor. */
FStaticMeshSceneProxy::FLODInfo::FLODInfo(const FStaticMeshSceneProxyDesc& InProxyDesc, const FStaticMeshVertexFactoriesArray& InLODVertexFactories, int32 LODIndex, int32 InClampedMinLOD, bool bLODsShareStaticLighting)
	: FLightCacheInterface()
	, OverrideColorVertexBuffer(nullptr)
	, bUsesMeshModifyingMaterials(false)
{
	const auto FeatureLevel =  InProxyDesc.Scene->GetFeatureLevel();
	const EShaderPlatform ShaderPlatform = InProxyDesc.Scene->GetShaderPlatform();

	FStaticMeshRenderData* MeshRenderData = InProxyDesc.GetStaticMesh()->GetRenderData();
	FStaticMeshLODResources& LODModel = MeshRenderData->LODResources[LODIndex];
	const FStaticMeshVertexFactories& VFs = InLODVertexFactories[LODIndex];

	if (InProxyDesc.LightmapType == ELightmapType::ForceVolumetric)
	{
		SetGlobalVolumeLightmap(true);
	}

	bool bUsePSOPrecacheFallbackMaterial = InProxyDesc.UsePSOPrecacheFallbackMaterial();

	bool bMeshMapBuildDataOverriddenByLightmapPreview = false;

	const UStaticMeshComponent* Component = InProxyDesc.GetUStaticMeshComponent();

#if WITH_EDITOR	
	// The component may not have corresponding FStaticMeshComponentLODInfo in its LODData, and that's why we're overriding MeshMapBuildData here (instead of inside GetMeshMapBuildData).
	if (Component && FStaticLightingSystemInterface::GetPrimitiveMeshMapBuildData(Component, LODIndex))
	{
		const FMeshMapBuildData* MeshMapBuildData = FStaticLightingSystemInterface::GetPrimitiveMeshMapBuildData(Component, LODIndex);
		if (MeshMapBuildData)
		{
			bMeshMapBuildDataOverriddenByLightmapPreview = true;

			SetLightMap(MeshMapBuildData->LightMap);
			SetShadowMap(MeshMapBuildData->ShadowMap);
			SetResourceCluster(MeshMapBuildData->ResourceCluster);
			bCanUsePrecomputedLightingParametersFromGPUScene = true;
			IrrelevantLights = MeshMapBuildData->IrrelevantLights;
		}
	}
#endif

	if (LODIndex < InProxyDesc.LODData.Num() && LODIndex >= InClampedMinLOD)
	{
		const FStaticMeshComponentLODInfo& ComponentLODInfo = InProxyDesc.LODData[LODIndex];

		if (!bMeshMapBuildDataOverriddenByLightmapPreview)
		{
			if (InProxyDesc.LightmapType != ELightmapType::ForceVolumetric && Component)
			{
				const FMeshMapBuildData* MeshMapBuildData = Component->GetMeshMapBuildData(ComponentLODInfo);
				if (MeshMapBuildData)
				{
					SetLightMap(MeshMapBuildData->LightMap);
					SetShadowMap(MeshMapBuildData->ShadowMap);
					SetResourceCluster(MeshMapBuildData->ResourceCluster);
					bCanUsePrecomputedLightingParametersFromGPUScene = true;
					IrrelevantLights = MeshMapBuildData->IrrelevantLights;
				}
			}
		}
		
		// Initialize this LOD's overridden vertex colors, if it has any
		if( ComponentLODInfo.OverrideVertexColors )
		{
			bool bBroken = false;
			for (int32 SectionIndex = 0; SectionIndex < LODModel.Sections.Num(); SectionIndex++)
			{
				const FStaticMeshSection& Section = LODModel.Sections[SectionIndex];
				if (Section.MaxVertexIndex >= ComponentLODInfo.OverrideVertexColors->GetNumVertices())
				{
					bBroken = true;
					break;
				}
			}
			if (!bBroken)
			{
				// the instance should point to the loaded data to avoid copy and memory waste
				OverrideColorVertexBuffer = ComponentLODInfo.OverrideVertexColors;
				check(OverrideColorVertexBuffer->GetStride() == sizeof(FColor)); //assumed when we set up the stream
				{
					TUniformBufferRef<FLocalVertexFactoryUniformShaderParameters>* UniformBufferPtr = &OverrideColorVFUniformBuffer;
					const FLocalVertexFactory* LocalVF = &VFs.VertexFactoryOverrideColorVertexBuffer;
					FColorVertexBuffer* VertexBuffer = OverrideColorVertexBuffer;

					//temp measure to identify nullptr crashes deep in the renderer
					FString ComponentPathName = InProxyDesc.GetPathName();
					checkf(LODModel.VertexBuffers.PositionVertexBuffer.GetNumVertices() > 0, TEXT("LOD: %i of PathName: %s has an empty position stream."), LODIndex, *ComponentPathName);
					
					ENQUEUE_RENDER_COMMAND(FLocalVertexFactoryCopyData)(
						[UniformBufferPtr, LocalVF, LODIndex, VertexBuffer, ComponentPathName](FRHICommandListImmediate& RHICmdList)
					{
						if (RHISupportsManualVertexFetch(GMaxRHIShaderPlatform))
						{
							checkf(LocalVF->GetTangentsSRV(), TEXT("LOD: %i of PathName: %s has a null tangents srv."), LODIndex, *ComponentPathName);
							checkf(LocalVF->GetTextureCoordinatesSRV(), TEXT("LOD: %i of PathName: %s has a null texcoord srv."), LODIndex, *ComponentPathName);
						}
						*UniformBufferPtr = CreateLocalVFUniformBuffer(LocalVF, LODIndex, VertexBuffer, 0, 0);
					});
				}
			}
		}
	}
	
	if (!bMeshMapBuildDataOverriddenByLightmapPreview)
	{
		if (LODIndex > 0
			&& bLODsShareStaticLighting
			&& InProxyDesc.LODData.IsValidIndex(0)
			&& InProxyDesc.LightmapType != ELightmapType::ForceVolumetric
			&& LODIndex >= InClampedMinLOD)
		{
			const FStaticMeshComponentLODInfo& ComponentLODInfo = InProxyDesc.LODData[0];
			const FMeshMapBuildData* MeshMapBuildData = Component ? Component->GetMeshMapBuildData(ComponentLODInfo) : nullptr;

			if (MeshMapBuildData)
			{
				SetLightMap(MeshMapBuildData->LightMap);
				SetShadowMap(MeshMapBuildData->ShadowMap);
				SetResourceCluster(MeshMapBuildData->ResourceCluster);
				bCanUsePrecomputedLightingParametersFromGPUScene = true;
				IrrelevantLights = MeshMapBuildData->IrrelevantLights;
			}
		}
	}

	const bool bHasSurfaceStaticLighting = GetLightMap() != NULL || GetShadowMap() != NULL;

	// Gather the materials applied to the LOD.
	Sections.Empty(MeshRenderData->LODResources[LODIndex].Sections.Num());
	
	TArray<TObjectPtr<UMaterialInterface>> ProxyMaterialSlotsOverlayMaterial;
	InProxyDesc.GetMaterialSlotsOverlayMaterial(ProxyMaterialSlotsOverlayMaterial);

	for(int32 SectionIndex = 0;SectionIndex < LODModel.Sections.Num();SectionIndex++)
	{
		const FStaticMeshSection& Section = LODModel.Sections[SectionIndex];
		FSectionInfo SectionInfo;

		// Determine the material applied to this element of the LOD.
		SectionInfo.Material = InProxyDesc.GetMaterial(Section.MaterialIndex);
		SectionInfo.OverlayMaterial = FMaterialOverlayHelper::GetOverlayMaterial(ProxyMaterialSlotsOverlayMaterial, Section.MaterialIndex);
		SectionInfo.MaterialIndex = Section.MaterialIndex;
		
		bool bUseDefaultMaterial = (GForceDefaultMaterial && SectionInfo.Material && !IsTranslucentBlendMode(*SectionInfo.Material));
		if (bUsePSOPrecacheFallbackMaterial || bUseDefaultMaterial)
		{
			UMaterial* FallbackMaterial = (bUseDefaultMaterial || UPSOPrecacheSettingsManager::GetFallbackMaterial() == nullptr) ? UMaterial::GetDefaultMaterial(MD_Surface) : UPSOPrecacheSettingsManager::GetFallbackMaterial();			
			SectionInfo.Material = FallbackMaterial;
			if (SectionInfo.OverlayMaterial)
			{
				SectionInfo.OverlayMaterial = FallbackMaterial;
			}
		}

		// If there isn't an applied material, or if the material doesn't have static mesh usage or if we need static lighting and the material doesn't support it, then fall back to the default material.
		if (!SectionInfo.Material || 
			!SectionInfo.Material->CheckMaterialUsage_Concurrent(MATUSAGE_StaticMesh) ||
			(bHasSurfaceStaticLighting && !SectionInfo.Material->CheckMaterialUsage_Concurrent(MATUSAGE_StaticLighting)))
		{
			SectionInfo.Material = UMaterial::GetDefaultMaterial(MD_Surface);
			if (SectionInfo.OverlayMaterial)
			{
				SectionInfo.OverlayMaterial = UMaterial::GetDefaultMaterial(MD_Surface);
			}
		}

		// Per-section selection for the editor.
#if WITH_EDITORONLY_DATA
		if (GIsEditor)
		{
			if (InProxyDesc.SelectedEditorMaterial >= 0)
			{
				SectionInfo.bSelected = (InProxyDesc.SelectedEditorMaterial == Section.MaterialIndex);
			}
			else
			{
				SectionInfo.bSelected = (InProxyDesc.SelectedEditorSection == SectionIndex);
			}
		}
#endif

		// Store the element info.
		Sections.Add(SectionInfo);

		// Flag the entire LOD if any material modifies its mesh
		FMaterialResource const* MaterialResource = const_cast<UMaterialInterface const*>(SectionInfo.Material)->GetMaterial_Concurrent()->GetMaterialResource(ShaderPlatform);
		if(MaterialResource)
		{
			if (IsInAnyRenderingThread())
			{
				if (MaterialResource->MaterialModifiesMeshPosition_RenderThread())
				{
					bUsesMeshModifyingMaterials = true;
				}
			}
			else
			{
				if (MaterialResource->MaterialModifiesMeshPosition_GameThread())
				{
					bUsesMeshModifyingMaterials = true;
				}
			}
		}
	}

}

// FLightCacheInterface.
FLightInteraction FStaticMeshSceneProxy::FLODInfo::GetInteraction(const FLightSceneProxy* LightSceneProxy) const
{
	// ask base class
	ELightInteractionType LightInteraction = GetStaticInteraction(LightSceneProxy, IrrelevantLights);

	if(LightInteraction != LIT_MAX)
	{
		return FLightInteraction(LightInteraction);
	}

	// Use dynamic lighting if the light doesn't have static lighting.
	return FLightInteraction::Dynamic();
}

FStaticMeshSceneProxy::FRayTracingLODInfo::FRayTracingLODInfo(const FStaticMeshSceneProxyDesc& InProxyDesc, int32 LODIndex)
{
	const auto FeatureLevel =  InProxyDesc.GetWorld()->GetFeatureLevel();

	FStaticMeshRenderData* MeshRenderData = InProxyDesc.GetStaticMesh()->GetRenderData();
	FStaticMeshRayTracingProxyLOD& LOD = MeshRenderData->RayTracingProxy->LODs[LODIndex];

	bool bUsePSOPrecacheFallbackMaterial = InProxyDesc.UsePSOPrecacheFallbackMaterial();

	// Gather the materials applied to the LOD.
	Sections.Empty(LOD.Sections->Num());
	for(int32 SectionIndex = 0;SectionIndex < LOD.Sections->Num();SectionIndex++)
	{
		const FStaticMeshSection& Section = (*LOD.Sections)[SectionIndex];
		FSectionInfo SectionInfo;

		// Determine the material applied to this element of the LOD.
		SectionInfo.Material = InProxyDesc.GetMaterial(Section.MaterialIndex);
		
		bool bUseDefaultMaterial = (GForceDefaultMaterial && SectionInfo.Material && !IsTranslucentBlendMode(*SectionInfo.Material));
		if (bUsePSOPrecacheFallbackMaterial || bUseDefaultMaterial)
		{
			SectionInfo.Material = (bUseDefaultMaterial || UPSOPrecacheSettingsManager::GetFallbackMaterial() == nullptr) ? UMaterial::GetDefaultMaterial(MD_Surface) : UPSOPrecacheSettingsManager::GetFallbackMaterial();
		}

		// If there isn't an applied material, fall back to the default material.
		if(!SectionInfo.Material)
		{
			SectionInfo.Material = UMaterial::GetDefaultMaterial(MD_Surface);
		}

		// Store the element info.
		Sections.Add(SectionInfo);
	}
}

float FStaticMeshSceneProxy::GetScreenSize( int32 LODIndex ) const
{
	return RenderData->ScreenSize[LODIndex].GetValue() * GetLodScreenSizeScale();
}

/**
 * Returns the LOD that the primitive will render at for this view. 
 *
 * @param Distance - distance from the current view to the component's bound origin
 */
int32 FStaticMeshSceneProxy::GetLOD(const FSceneView* View) const 
{
	if (ensureMsgf(RenderData, TEXT("StaticMesh [%s] missing RenderData."),
		(STATICMESH_ENABLE_DEBUG_RENDERING && StaticMesh) ? *StaticMesh->GetName() : TEXT("None")))
	{
		int32 CVarForcedLODLevel = GetCVarForceLOD_AnyThread();

		//If a LOD is being forced, use that one
		if (CVarForcedLODLevel >= 0)
		{
			return FMath::Clamp<int32>(CVarForcedLODLevel, 0, RenderData->LODResources.Num() - 1);
		}

		if (ForcedLodModel > 0)
		{
			return FMath::Clamp(ForcedLodModel, ClampedMinLOD + 1, RenderData->LODResources.Num()) - 1;
		}
	}

#if WITH_EDITOR
	if (View->Family && View->Family->EngineShowFlags.LOD == 0)
	{
		return 0;
	}
#endif

	const FBoxSphereBounds& ProxyBounds = GetBounds();
	const float LODScale = GetCachedScalabilityCVars().StaticMeshLODDistanceScale * GetLodScreenSizeScale();
	return ComputeStaticMeshLOD(RenderData, ProxyBounds.Origin, ProxyBounds.SphereRadius, *View, ClampedMinLOD, LODScale);
}

FLODMask FStaticMeshSceneProxy::GetLODMask(const FSceneView* View) const
{
	FLODMask Result;

	if (!ensureMsgf(RenderData, TEXT("StaticMesh [%s] missing RenderData."),
		(STATICMESH_ENABLE_DEBUG_RENDERING && StaticMesh) ? *StaticMesh->GetName() : TEXT("None")))
	{
		Result.SetLOD(0);
	}
	else
	{
		int32 CVarForcedLODLevel = GetCVarForceLOD();

		//If a LOD is being forced, use that one
		if (CVarForcedLODLevel >= 0)
		{
			Result.SetLOD(FMath::Clamp<int32>(CVarForcedLODLevel, ClampedMinLOD, RenderData->LODResources.Num() - 1));
		}
		else if (View->DrawDynamicFlags & EDrawDynamicFlags::ForceLowestLOD)
		{
			Result.SetLOD(RenderData->LODResources.Num() - 1);
		}
		else if (ForcedLodModel > 0)
		{
			Result.SetLOD(FMath::Clamp(ForcedLodModel, ClampedMinLOD + 1, RenderData->LODResources.Num()) - 1);
		}
#if WITH_EDITOR
		else if (View->Family && View->Family->EngineShowFlags.LOD == 0)
		{
			Result.SetLOD(0);
		}
#endif
		else
		{
			const FBoxSphereBounds& ProxyBounds = GetBounds();
			bool bUseDithered = false;
			if (LODs.Num())
			{
				checkSlow(RenderData);

				// only dither if at least one section in LOD0 is dithered. Mixed dithering on sections won't work very well, but it makes an attempt
				const ERHIFeatureLevel::Type FeatureLevel = GetScene().GetFeatureLevel();
				const FLODInfo& ProxyLODInfo = LODs[0];
				const FStaticMeshLODResources& LODModel = RenderData->LODResources[0];
				// Draw the static mesh elements.
				for(int32 SectionIndex = 0; SectionIndex < LODModel.Sections.Num(); SectionIndex++)
				{
					const FMaterial& Material = ProxyLODInfo.Sections[SectionIndex].Material->GetRenderProxy()->GetIncompleteMaterialWithFallback(FeatureLevel);
					if (Material.IsDitheredLODTransition())
					{
						bUseDithered = true;
						break;
					}
				}

			}

			FCachedSystemScalabilityCVars CachedSystemScalabilityCVars = GetCachedScalabilityCVars();
			const float LODScale = CachedSystemScalabilityCVars.StaticMeshLODDistanceScale * GetLodScreenSizeScale();

			if (bUseDithered)
			{
				for (int32 Sample = 0; Sample < 2; Sample++)
				{
					Result.SetLODSample(ComputeTemporalStaticMeshLOD(RenderData, ProxyBounds.Origin, ProxyBounds.SphereRadius, *View, ClampedMinLOD, LODScale, Sample), Sample);
				}
			}
			else
			{
				Result.SetLOD(ComputeStaticMeshLOD(RenderData, ProxyBounds.Origin, ProxyBounds.SphereRadius, *View, ClampedMinLOD, LODScale));
			}
		}
	}

	const int8 CurFirstLODIdx = GetCurrentFirstLODIdx_Internal();
	check(CurFirstLODIdx >= 0);
	Result.ClampToFirstLOD(CurFirstLODIdx);
	
	return Result;
}

bool UStaticMeshComponent::ShouldCreateNaniteProxy(Nanite::FMaterialAudit* OutNaniteMaterials) const
{
	return Nanite::FNaniteResourcesHelper::ShouldCreateNaniteProxy(*this, OutNaniteMaterials);
}

FPrimitiveSceneProxy* UStaticMeshComponent::CreateStaticMeshSceneProxy(Nanite::FMaterialAudit& NaniteMaterials, bool bCreateNanite)
{
	// Default implementation: Nanite::FSceneProxy or FStaticMeshSceneProxy

	LLM_SCOPE(ELLMTag::StaticMesh);

	if (bCreateNanite)
	{
		return ::new Nanite::FSceneProxy(NaniteMaterials, this);
	}
	
	auto* Proxy = ::new FStaticMeshSceneProxy(this, false);
#if STATICMESH_ENABLE_DEBUG_RENDERING
	SendRenderDebugPhysics(Proxy);
#endif

	return Proxy;
}

FPrimitiveSceneProxy* UStaticMeshComponent::CreateSceneProxy()
{
	return FStaticMeshComponentHelper::CreateSceneProxy(*this);
}

bool UStaticMeshComponent::ShouldRecreateProxyOnUpdateTransform() const
{
	return (Mobility != EComponentMobility::Movable);
}
