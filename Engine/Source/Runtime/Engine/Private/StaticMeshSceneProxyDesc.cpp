// Copyright Epic Games, Inc. All Rights Reserved.

#include "StaticMeshSceneProxyDesc.h"

#include "MaterialCache/MaterialCacheVirtualTexture.h"
#include "PhysicsEngine/BodySetup.h"
#include "SceneInterface.h"
#include "StaticMeshComponentHelper.h"

FStaticMeshSceneProxyDesc::FStaticMeshSceneProxyDesc()
{
	CastShadow = true;
	bUseAsOccluder = true;

	// bitfields init (until we switch to c++20)
	bReverseCulling = false;
#if STATICMESH_ENABLE_DEBUG_RENDERING
	bDrawMeshCollisionIfComplex = false;
	bDrawMeshCollisionIfSimple = false;
#endif
	bEvaluateWorldPositionOffset = true;
	bOverrideMinLOD = false;

	bCastDistanceFieldIndirectShadow = false;
	bOverrideDistanceFieldSelfShadowBias = false;
	bEvaluateWorldPositionOffsetInRayTracing = false;
	bSortTriangles = false;

	bDisplayNaniteFallbackMesh = false;
	bDisallowNanite = false;
	bForceDisableNanite = false;
	bForceNaniteForMasked = false;

	bUseProvidedMaterialRelevance = false;
}

FStaticMeshSceneProxyDesc::FStaticMeshSceneProxyDesc(const UStaticMeshComponent* InComponent)
	: FStaticMeshSceneProxyDesc()
{
	InitializeFromStaticMeshComponent(InComponent);
}

FStaticMeshSceneProxyDesc::~FStaticMeshSceneProxyDesc() = default;

void FStaticMeshSceneProxyDesc::InitializeFromStaticMeshComponent(const UStaticMeshComponent* InComponent)
{
	InitializeFromPrimitiveComponent(InComponent);

	StaticMesh = InComponent->GetStaticMesh();
	OverrideMaterials = const_cast<UStaticMeshComponent*>(InComponent)->OverrideMaterials;
	OverlayMaterial = InComponent->GetOverlayMaterial();
	OverlayMaterialMaxDrawDistance = InComponent->GetOverlayMaterialMaxDrawDistance();
	InComponent->GetMaterialSlotsOverlayMaterial(MaterialSlotsOverlayMaterial);

	ForcedLodModel = InComponent->ForcedLodModel;
	MinLOD = InComponent->MinLOD;
	WorldPositionOffsetDisableDistance = InComponent->WorldPositionOffsetDisableDistance;
	NanitePixelProgrammableDistance = InComponent->NanitePixelProgrammableDistance;
	bReverseCulling = InComponent->bReverseCulling;
	bEvaluateWorldPositionOffset = InComponent->bEvaluateWorldPositionOffset;
	bOverrideMinLOD = InComponent->bOverrideMinLOD;
	bCastDistanceFieldIndirectShadow = InComponent->bCastDistanceFieldIndirectShadow;
	bOverrideDistanceFieldSelfShadowBias = InComponent->bOverrideDistanceFieldSelfShadowBias;
	bEvaluateWorldPositionOffsetInRayTracing = InComponent->bEvaluateWorldPositionOffsetInRayTracing;
	bSortTriangles = InComponent->bSortTriangles;
#if WITH_EDITOR
	bDisplayNaniteFallbackMesh = InComponent->bDisplayNaniteFallbackMesh;
#endif
	bDisallowNanite = InComponent->bDisallowNanite;
	bForceDisableNanite = InComponent->bForceDisableNanite;
	bForceNaniteForMasked = InComponent->bForceNaniteForMasked;
	DistanceFieldSelfShadowBias = InComponent->DistanceFieldSelfShadowBias;
	DistanceFieldIndirectShadowMinVisibility = InComponent->DistanceFieldIndirectShadowMinVisibility;
	StaticLightMapResolution = InComponent->GetStaticLightMapResolution();
	LightmapType = InComponent->GetLightmapType();

#if WITH_EDITORONLY_DATA
	StreamingDistanceMultiplier = InComponent->StreamingDistanceMultiplier;
	MaterialStreamingRelativeBoxes = const_cast<UStaticMeshComponent*>(InComponent)->MaterialStreamingRelativeBoxes;
	SectionIndexPreview = InComponent->SectionIndexPreview;
	MaterialIndexPreview = InComponent->MaterialIndexPreview;
	SelectedEditorMaterial = InComponent->SelectedEditorMaterial;
	SelectedEditorSection = InComponent->SelectedEditorSection;

	TextureStreamingTransformScale = InComponent->GetTextureStreamingTransformScale();
#endif

	NaniteResources = InComponent->GetNaniteResources();
	BodySetup = const_cast<UStaticMeshComponent*>(InComponent)->GetBodySetup();

#if STATICMESH_ENABLE_DEBUG_RENDERING
	const bool bHasCollisionState = BodySetup && !BodySetup->bNeverNeedsCookedCollisionData;
	bDrawMeshCollisionIfComplex = InComponent->bDrawMeshCollisionIfComplex && bHasCollisionState;
	bDrawMeshCollisionIfSimple = InComponent->bDrawMeshCollisionIfSimple && bHasCollisionState;
#endif

	LODData = const_cast<UStaticMeshComponent*>(InComponent)->LODData;

	WireframeColor = InComponent->GetWireframeColor();
	LODParentPrimitive = InComponent->GetLODParentPrimitive();

	if (GetScene())
	{
		SetMaterialRelevance(InComponent->GetMaterialRelevance(GetScene()->GetShaderPlatform()));
	}
	SetCollisionResponseToChannels(InComponent->GetCollisionResponseToChannels());

	MeshPaintTexture = InComponent->MeshPaintTextureOverride ? InComponent->MeshPaintTextureOverride.Get() : InComponent->GetMeshPaintTexture();
	MeshPaintTextureCoordinateIndex = InComponent->GetMeshPaintTextureCoordinateIndex();

	MaterialCacheTextures = InComponent->MaterialCacheTextures;
	MaterialCacheUVRegion = InComponent->MaterialCacheUVRegion;
	MaterialCacheTextureCoordinateIndex = InComponent->MaterialCacheUVCoordinateIndex;
}

void FStaticMeshSceneProxyDesc::GetUsedMaterials(TArray<UMaterialInterface*>& OutMaterials, bool bGetDebugMaterials) const
{
	FStaticMeshComponentHelper::GetUsedMaterials(*this, OutMaterials, bGetDebugMaterials);
}

UMaterialInterface* FStaticMeshSceneProxyDesc::GetMaterial(int32 MaterialIndex, bool bDoingNaniteMaterialAudit, bool bIgnoreNaniteOverrideMaterials) const
{
	return FStaticMeshComponentHelper::GetMaterial(*this, MaterialIndex, bDoingNaniteMaterialAudit, bIgnoreNaniteOverrideMaterials);
}

int32 FStaticMeshSceneProxyDesc::GetNumMaterials() const
{
	return GetStaticMesh() ? GetStaticMesh()->GetStaticMaterials().Num() : 0;
}

bool FStaticMeshSceneProxyDesc::ShouldCreateMaterialCacheProxy() const
{
	return !MaterialCacheTextures.IsEmpty();
}

const UStaticMeshComponent* FStaticMeshSceneProxyDesc::GetUStaticMeshComponent() const
{
	return Cast<UStaticMeshComponent>(Component);
}

void FStaticMeshSceneProxyDesc::GetMaterialSlotsOverlayMaterial(TArray<TObjectPtr<UMaterialInterface>>& OutMaterialSlotsOverlayMaterial) const
{
	OutMaterialSlotsOverlayMaterial = MaterialSlotsOverlayMaterial;
}

void FStaticMeshSceneProxyDesc::SetMaterialRelevance(const FMaterialRelevance& InRelevance)
{
	MaterialRelevance = InRelevance;
	bUseProvidedMaterialRelevance = true;
}

FMaterialRelevance FStaticMeshSceneProxyDesc::GetMaterialRelevance(EShaderPlatform InShaderPlatform) const
{
	if (bUseProvidedMaterialRelevance)
	{
		return MaterialRelevance;
	}

	return FMeshComponentHelper::GetMaterialRelevance(*this, InShaderPlatform);
}

// Deprecated in 5.7
FMaterialRelevance FStaticMeshSceneProxyDesc::GetMaterialRelevance(ERHIFeatureLevel::Type InFeatureLevel) const
{
	return GetMaterialRelevance(GetFeatureLevelShaderPlatform_Checked(InFeatureLevel));
}

FTextureResource* FStaticMeshSceneProxyDesc::GetMeshPaintTextureResource() const
{
	// Prefer the GT-resolved cache when available. Callers that build the desc on a thread
	// where UTexture::GetResource() is not safe (e.g. FastGeo's cooked async render-state
	// worker) populate this on the game thread before dispatching the worker.
	if (CachedMeshPaintTextureResource)
	{
		return CachedMeshPaintTextureResource;
	}
	if (MeshPaintTexture && MeshPaintTexture->IsCurrentlyVirtualTextured())
	{
		return MeshPaintTexture->GetResource();
	}
	return nullptr;
}

UBodySetup* FStaticMeshSceneProxyDesc::GetBodySetup() const
{
	if (BodySetup)
	{
		return BodySetup;
	}

	if (GetStaticMesh())
	{
		return GetStaticMesh()->GetBodySetup();
	}

	return nullptr;
}

bool FStaticMeshSceneProxyDesc::HasValidNaniteData() const
{
	return Nanite::FNaniteResourcesHelper::HasValidNaniteData(*this);
}

bool FStaticMeshSceneProxyDesc::ShouldCreateNaniteProxy(Nanite::FMaterialAudit* OutNaniteMaterials /*= nullptr*/) const
{
	return Nanite::FNaniteResourcesHelper::ShouldCreateNaniteProxy(*this, OutNaniteMaterials);
}

bool FStaticMeshSceneProxyDesc::UseNaniteOverrideMaterials(bool bDoingMaterialAudit) const
{
	return Nanite::FNaniteResourcesHelper::UseNaniteOverrideMaterials(*this, bDoingMaterialAudit);
}

UMaterialInterface* FStaticMeshSceneProxyDesc::GetNaniteAuditMaterial(int32 MaterialIndex) const
{
	return GetMaterial(MaterialIndex, true);
}

void FStaticMeshSceneProxyDesc::SetCollisionResponseToChannels(const FCollisionResponseContainer& InContainer)
{
	if (&InContainer != &FCollisionResponseContainer::GetDefaultResponseContainer())
	{
		CollisionResponseContainer = InContainer;
	}
}

const FCollisionResponseContainer& FStaticMeshSceneProxyDesc::GetCollisionResponseToChannels() const
{
	if (CollisionResponseContainer.IsSet())
	{
		return *CollisionResponseContainer;
	}

	return FCollisionResponseContainer::GetDefaultResponseContainer();
}
