// Copyright Epic Games, Inc. All Rights Reserved.

#include "Engine/StaticMeshSourceData.h"
#include "Engine/StaticMesh.h"
#include "Serialization/MemoryReader.h"
#include "Serialization/MemoryWriter.h"
#include "StaticMeshDescription.h"
#include "StaticMeshOperations.h"
#include "ProfilingDebugging/CookStats.h"
#include "UObject/EditorObjectVersion.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(StaticMeshSourceData)

#if WITH_EDITOR
#include "DerivedDataCacheInterface.h"
#include "RawMesh.h"
#endif


UStaticMeshDescriptionBulkData::UStaticMeshDescriptionBulkData()
{
#if WITH_EDITORONLY_DATA
	const bool bTransient = true;
	PreallocatedMeshDescription = CreateDefaultSubobject<UStaticMeshDescription>(TEXT("MeshDescription"), bTransient);
#endif
}


FStaticMeshSourceModel::FStaticMeshSourceModel()
{
#if WITH_EDITOR
	RawMeshBulkData = new FRawMeshBulkData();
#endif // #if WITH_EDITOR
	LODDistance_DEPRECATED = 0.0f;
	ScreenSize.Default = 0.0f;
#if WITH_EDITORONLY_DATA
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	StaticMeshOwner = nullptr;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	bImportWithBaseMesh = false;
	StaticMeshDescriptionBulkData = nullptr;
#endif
}

FStaticMeshSourceModel::~FStaticMeshSourceModel()
{
#if WITH_EDITOR
	if (RawMeshBulkData)
	{
		delete RawMeshBulkData;
	}
#endif // #if WITH_EDITOR
}

FStaticMeshSourceModel::FStaticMeshSourceModel(FStaticMeshSourceModel&& Other)
{
	*this = MoveTemp(Other);
}

FStaticMeshSourceModel& FStaticMeshSourceModel::operator=(FStaticMeshSourceModel&& Other)
{
	if (this == &Other)
	{
		return *this;
	}

#if WITH_EDITOR
	RawMeshBulkData = Other.RawMeshBulkData;
	Other.RawMeshBulkData = nullptr;
#endif

#if WITH_EDITORONLY_DATA
	{
		FCriticalSection* First;
		FCriticalSection* Second;
		// Lock always in the same order
		if (this < &Other)
		{
			First = &StaticMeshDescriptionBulkDataCS;
			Second = &Other.StaticMeshDescriptionBulkDataCS;
		}
		else
		{
			First = &Other.StaticMeshDescriptionBulkDataCS;
			Second = &StaticMeshDescriptionBulkDataCS;
		}

		FScopeLock StaticMeshDescriptionBulkDataLock(First);
		FScopeLock OtherStaticMeshDescriptionBulkDataLock(Second);

		StaticMeshDescriptionBulkData = Other.StaticMeshDescriptionBulkData;
		Other.StaticMeshDescriptionBulkData = nullptr;
	}

	bImportWithBaseMesh = Other.bImportWithBaseMesh;
#endif

	BuildSettings = MoveTemp(Other.BuildSettings);
	ReductionSettings = MoveTemp(Other.ReductionSettings);
	LODDistance_DEPRECATED = Other.LODDistance_DEPRECATED;
	ScreenSize = MoveTemp(Other.ScreenSize);
	SourceImportFilename = MoveTemp(Other.SourceImportFilename);

	return *this;
}


void FStaticMeshSourceModel::CreateSubObjects(UStaticMesh* InOwner)
{
#if WITH_EDITORONLY_DATA
	FScopeLock StaticMeshDescriptionBulkDataLock(&StaticMeshDescriptionBulkDataCS);

	if (StaticMeshDescriptionBulkData == nullptr)
	{
		// Currently there is a restriction on creating UObjects on a thread while garbage collection is taking place,
		// so use a GCScopeGuard to prevent it happening at the same time.
		FGCScopeGuard Guard;
		StaticMeshDescriptionBulkData = NewObject<UStaticMeshDescriptionBulkData>(InOwner, NAME_None, RF_Transactional);

		// OK to clear the async flag immediately here, because the UObject* has already been assigned directly to a UPROPERTY.
		StaticMeshDescriptionBulkData->AtomicallyClearInternalFlags(EInternalObjectFlags::Async);
	}

	check(StaticMeshDescriptionBulkData->GetMeshDescription() == nullptr);
#endif
}


#if WITH_EDITOR
bool FStaticMeshSourceModel::IsRawMeshEmpty() const
{
	FScopeLock StaticMeshDescriptionBulkDataLock(&StaticMeshDescriptionBulkDataCS);

	// The RawMeshBulkData will always be empty, so the test here is whether we are able to construct a valid non-empty RawMesh
	// from what exists in the StaticMeshDescription and its bulk data.
	check(RawMeshBulkData->IsEmpty());
	check(StaticMeshDescriptionBulkData != nullptr);

	return !StaticMeshDescriptionBulkData->IsBulkDataValid() && !StaticMeshDescriptionBulkData->HasCachedMeshDescription();
}

void FStaticMeshSourceModel::LoadRawMesh(FRawMesh& OutRawMesh) const
{
	FScopeLock StaticMeshDescriptionBulkDataLock(&StaticMeshDescriptionBulkDataCS);

	check(RawMeshBulkData->IsEmpty());
	check(StaticMeshDescriptionBulkData != nullptr);

	FStaticMeshSourceModel* MutableThis = const_cast<FStaticMeshSourceModel*>(this);
	if (FMeshDescription* CachedMeshDescription = MutableThis->GetOrCacheMeshDescription())
	{
		TMap<FName, int32> MaterialMap;
		UStaticMesh* StaticMesh = GetStaticMeshOwner();
		check(StaticMesh != nullptr);
		for (int32 MaterialIndex = 0; MaterialIndex < StaticMesh->GetStaticMaterials().Num(); ++MaterialIndex)
		{
			MaterialMap.Add(StaticMesh->GetStaticMaterials()[MaterialIndex].ImportedMaterialSlotName, MaterialIndex);
		}
		FStaticMeshOperations::ConvertToRawMesh(*CachedMeshDescription, OutRawMesh, MaterialMap);
	}
}

void FStaticMeshSourceModel::SaveRawMesh(FRawMesh& InRawMesh, bool /* unused */)
{
	if (!InRawMesh.IsValid())
	{
		return;
	}

	TRACE_CPUPROFILER_EVENT_SCOPE(FStaticMeshSourceModel::SaveRawMesh);

	FScopeLock StaticMeshDescriptionBulkDataLock(&StaticMeshDescriptionBulkDataCS);

	check(RawMeshBulkData->IsEmpty());
	check(StaticMeshDescriptionBulkData != nullptr);

	UMeshDescriptionBase* MeshDescriptionBase = StaticMeshDescriptionBulkData->CreateMeshDescription();

	TMap<int32, FName> MaterialMap;
	FillMaterialName(MaterialMap);
	FStaticMeshOperations::ConvertFromRawMesh(InRawMesh, MeshDescriptionBase->GetMeshDescription(), MaterialMap);

	// Package up mesh description into bulk data
	const bool bUseHashAsGuid = false;
	StaticMeshDescriptionBulkData->CommitMeshDescription(bUseHashAsGuid);
}


UStaticMesh* FStaticMeshSourceModel::GetStaticMeshOwner() const
{
	check(StaticMeshDescriptionBulkData != nullptr);
	return Cast<UStaticMesh>(StaticMeshDescriptionBulkData->GetOuter());
}

bool FStaticMeshSourceModel::IsSourceModelInitialized() const
{
	FScopeLock StaticMeshDescriptionBulkDataLock(&StaticMeshDescriptionBulkDataCS);

	if (StaticMeshDescriptionBulkData == nullptr)
	{
		return false;
	}
	if (StaticMeshDescriptionBulkData->IsBulkDataValid() == false)
	{
		return false;
	}

	return true;
}


bool FStaticMeshSourceModel::LoadMeshDescription(FMeshDescription& OutMeshDescription) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FStaticMeshSourceModel::LoadMeshDescription);

	FScopeLock StaticMeshDescriptionBulkDataLock(&StaticMeshDescriptionBulkDataCS);

	check(StaticMeshDescriptionBulkData != nullptr);

	// Ensure default MeshDescription result is empty, with no attributes registered
	OutMeshDescription = FMeshDescription();

	// If we have valid MeshDescriptionBulkData, unpack it and return it...
	if (StaticMeshDescriptionBulkData->IsBulkDataValid())
	{
		// Unpack MeshDescription from the bulk data which was deserialized. If there is an error deserializing, then the mesh description will be empty.
		StaticMeshDescriptionBulkData->GetBulkData().LoadMeshDescription(OutMeshDescription);
		return !OutMeshDescription.IsEmpty();
	}

	// RawMeshBulkData should always be empty now (soon to be deprecated)
	check(RawMeshBulkData->IsEmpty());

	// This LOD doesn't have a MeshDescriptionBulkData, or a RawMesh, so we presume that it's a generated LOD (and return false)
	return false;
}


bool FStaticMeshSourceModel::CloneMeshDescription(FMeshDescription& OutMeshDescription) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FStaticMeshSourceModel::CloneMeshDescription);

	FScopeLock StaticMeshDescriptionBulkDataLock(&StaticMeshDescriptionBulkDataCS);

	check(StaticMeshDescriptionBulkData != nullptr);
	if (StaticMeshDescriptionBulkData->HasCachedMeshDescription())
	{
		OutMeshDescription = StaticMeshDescriptionBulkData->GetMeshDescription()->GetMeshDescription();
		return true;
	}

	return LoadMeshDescription(OutMeshDescription);
}


FMeshDescription* FStaticMeshSourceModel::GetOrCacheMeshDescription()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FStaticMeshSourceModel::GetMeshDescription);

	FScopeLock StaticMeshDescriptionBulkDataLock(&StaticMeshDescriptionBulkDataCS);

	check(StaticMeshDescriptionBulkData != nullptr);

	if (!StaticMeshDescriptionBulkData->HasCachedMeshDescription())
	{
		FMeshDescription MeshDescription;
		if (LoadMeshDescription(MeshDescription))
		{
			UMeshDescriptionBase* MeshDescriptionBase = StaticMeshDescriptionBulkData->CreateMeshDescription();
			MeshDescriptionBase->SetMeshDescription(MoveTemp(MeshDescription));
		}
	}

	if (StaticMeshDescriptionBulkData->HasCachedMeshDescription())
	{
		return &StaticMeshDescriptionBulkData->GetMeshDescription()->GetMeshDescription();
	}
	else
	{
		return nullptr;
	}
}


FMeshDescription* FStaticMeshSourceModel::GetCachedMeshDescription() const
{
	FScopeLock StaticMeshDescriptionBulkDataLock(&StaticMeshDescriptionBulkDataCS);

	check(StaticMeshDescriptionBulkData != nullptr);
	if (StaticMeshDescriptionBulkData->HasCachedMeshDescription())
	{
		return &StaticMeshDescriptionBulkData->GetMeshDescription()->GetMeshDescription();
	}

	return nullptr;
}


UStaticMeshDescription* FStaticMeshSourceModel::GetCachedStaticMeshDescription() const
{
	FScopeLock StaticMeshDescriptionBulkDataLock(&StaticMeshDescriptionBulkDataCS);

	check(StaticMeshDescriptionBulkData != nullptr);
	return Cast<UStaticMeshDescription>(StaticMeshDescriptionBulkData->GetMeshDescription());
}


FMeshDescriptionBulkData* FStaticMeshSourceModel::GetMeshDescriptionBulkData() const
{
	FScopeLock StaticMeshDescriptionBulkDataLock(&StaticMeshDescriptionBulkDataCS);
	check(StaticMeshDescriptionBulkData != nullptr);
	return &StaticMeshDescriptionBulkData->GetBulkData();
}


bool FStaticMeshSourceModel::IsMeshDescriptionValid() const
{
	FScopeLock StaticMeshDescriptionBulkDataLock(&StaticMeshDescriptionBulkDataCS);

	// Determine whether a mesh description is valid without requiring it to be loaded first.
	// If there is a valid MeshDescriptionBulkData, we know this implies a valid mesh description.
	check(StaticMeshDescriptionBulkData != nullptr);
	check(RawMeshBulkData->IsEmpty());
	return StaticMeshDescriptionBulkData->HasCachedMeshDescription() || StaticMeshDescriptionBulkData->IsBulkDataValid();
}


FMeshDescription* FStaticMeshSourceModel::CreateMeshDescription()
{
	FScopeLock StaticMeshDescriptionBulkDataLock(&StaticMeshDescriptionBulkDataCS);
	check(StaticMeshDescriptionBulkData != nullptr);
	return &StaticMeshDescriptionBulkData->CreateMeshDescription()->GetMeshDescription();
}


void FStaticMeshSourceModel::CommitMeshDescription(bool bUseHashAsGuid)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FStaticMeshSourceModel::CommitMeshDescription);

	FScopeLock StaticMeshDescriptionBulkDataLock(&StaticMeshDescriptionBulkDataCS);

	check(RawMeshBulkData->IsEmpty());

	// This part should remain thread-safe so it can be called from any thread
	// as long as no more than one thread is calling it for the same UStaticMesh.

	check(StaticMeshDescriptionBulkData != nullptr);

	//Reset the mesh tris/verts count cache
	CacheMeshDescriptionTrianglesCount = MAX_uint32;
	CacheMeshDescriptionVerticesCount = MAX_uint32;
	if (StaticMeshDescriptionBulkData->HasCachedMeshDescription())
	{
		// Package up mesh description into bulk data
		StaticMeshDescriptionBulkData->CommitMeshDescription(bUseHashAsGuid);
		//Set the tris/verts count cache
		if (FMeshDescription* MeshDescription = GetCachedMeshDescription())
		{
			CacheMeshDescriptionTrianglesCount = static_cast<uint32>(MeshDescription->Triangles().Num());
			CacheMeshDescriptionVerticesCount = static_cast<uint32>(FStaticMeshOperations::GetUniqueVertexCount(*MeshDescription));
		}
	}
	else
	{
		StaticMeshDescriptionBulkData->Empty();
	}
}

void FStaticMeshSourceModel::ClearMeshDescription()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UStaticMesh::ClearMeshDescription);

	FScopeLock StaticMeshDescriptionBulkDataLock(&StaticMeshDescriptionBulkDataCS);

	check(StaticMeshDescriptionBulkData != nullptr);
	StaticMeshDescriptionBulkData->RemoveMeshDescription();
}

void FStaticMeshSourceModel::ResetReductionSetting()
{
	ReductionSettings.MaxDeviation = 0.0f;
	ReductionSettings.PercentTriangles = 1.f;
	ReductionSettings.MaxNumOfTriangles = MAX_uint32;
	ReductionSettings.PercentVertices = 1.f;
	ReductionSettings.MaxNumOfVerts = MAX_uint32;
}

void FStaticMeshSourceModel::SerializeBulkData(FArchive& Ar, UObject* Owner)
{
	// Initialize the StaticMeshDescriptionBulkData
	if (Ar.IsLoading())
	{
		// If this was a legacy asset, or is being created for the first time, create a bulkdata UObject wrapper
		if (StaticMeshDescriptionBulkData == nullptr)
		{
			// For now, we assume that serialization will only occur on the game thread
			ensure(IsInGameThread() || IsInAsyncLoadingThread() || IsInParallelLoadingThread());
			StaticMeshDescriptionBulkData = NewObject<UStaticMeshDescriptionBulkData>(Owner, NAME_None, RF_Transactional);
		}
	}

	if (Ar.IsLoading() && Ar.CustomVer(FEditorObjectVersion::GUID) < FEditorObjectVersion::StaticMeshDeprecatedRawMesh)
	{
		// If loading a legacy asset with RawMesh bulk data, serialize it here
		// The conversion to MeshDescription will be done asynchronously, during PostLoad.
		check(RawMeshBulkData != NULL);
		RawMeshBulkData->Serialize(Ar, Owner);
	}
	else
	{
		if (Ar.IsLoading())
		{
			if (Ar.CustomVer(FUE5MainStreamObjectVersion::GUID) < FUE5MainStreamObjectVersion::SerializeMeshDescriptionBase)
			{
				// If there's an inline mesh description bulk data (legacy version), serialize it here, and copy it into the new separate inner object
				bool bIsValid;
				Ar << bIsValid;

				if (bIsValid)
				{
					StaticMeshDescriptionBulkData->GetBulkData().Serialize(Ar, Owner);
				}
			}

		}
	}
}


void FStaticMeshSourceModel::FillMaterialName(TMap<int32, FName>& OutMaterialMap) const
{
	check(StaticMeshDescriptionBulkData != nullptr);
	
	UStaticMesh* StaticMesh = GetStaticMeshOwner();
	check(StaticMesh != nullptr);

	const TArray<FStaticMaterial>& StaticMaterials = StaticMesh->GetStaticMaterials();

	OutMaterialMap.Empty(StaticMaterials.Num());

	for (int32 MaterialIndex = 0; MaterialIndex < StaticMaterials.Num(); ++MaterialIndex)
	{
		FName MaterialName = StaticMaterials[MaterialIndex].ImportedMaterialSlotName;
		if (MaterialName == NAME_None)
		{
			MaterialName = *(TEXT("MaterialSlot_") + FString::FromInt(MaterialIndex));
		}
		OutMaterialMap.Add(MaterialIndex, MaterialName);
	}
}


// This is the key which is used for DDC data when legacy RawMesh is converted to MeshDescription.
// If static mesh derived data needs to be rebuilt (new format, serialization differences, etc.) replace the version GUID below with a new one.
// In case of merge conflicts with DDC versions, you MUST generate a new GUID and set this new GUID as the version.
#define MESHDATAKEY_STATICMESH_DERIVEDDATA_VER TEXT("FF50B08BF5961A71C7C1BD9D2AA6BD34")

static bool GetMeshDataKey(FString& OutKey, const FRawMeshBulkData* RawMeshBulkData, int LodIndex)
{
	OutKey.Empty();

	FSHA1 Sha;

	check(!RawMeshBulkData->IsEmpty());
	FString LodIndexString = FString::Printf(TEXT("%d_"), LodIndex);
	LodIndexString += RawMeshBulkData->GetIdString();

	const TArray<TCHAR, FString::AllocatorType>& LodIndexArray = LodIndexString.GetCharArray();
	Sha.Update((uint8*)LodIndexArray.GetData(), LodIndexArray.Num() * LodIndexArray.GetTypeSize());
	Sha.Final();

	// Retrieve the hash and use it to construct a pseudo-GUID.
	uint32 Hash[5];
	Sha.GetHash((uint8*)Hash);
	FGuid Guid = FGuid(Hash[0] ^ Hash[4], Hash[1], Hash[2], Hash[3]);
	FString MeshLodData = Guid.ToString(EGuidFormats::Digits);

	OutKey = FDerivedDataCacheInterface::BuildCacheKey(
		TEXT("MESHDATAKEY_STATICMESH"),
		MESHDATAKEY_STATICMESH_DERIVEDDATA_VER,
		*MeshLodData
	);
	return true;
}


#if ENABLE_COOK_STATS
namespace StaticMeshConvertStats
{
	static FCookStats::FDDCResourceUsageStats UsageStats;
	static FCookStatsManager::FAutoRegisterCallback RegisterCookStats([](FCookStatsManager::AddStatFuncRef AddStat)
		{
			UsageStats.LogStats(AddStat, TEXT("StaticMeshConvert.Usage"), TEXT(""));
		});
}
#endif


void FStaticMeshSourceModel::ConvertRawMesh(int32 LodIndex)
{
	check(StaticMeshDescriptionBulkData != nullptr);
	UStaticMesh* StaticMesh = GetStaticMeshOwner();
	check(StaticMesh != nullptr);

	if (!RawMeshBulkData->IsEmpty() && !StaticMeshDescriptionBulkData->IsBulkDataValid())
	{
		FString MeshDataKey;
		if (GetMeshDataKey(MeshDataKey, RawMeshBulkData, LodIndex))
		{
			COOK_STAT(auto Timer = StaticMeshConvertStats::UsageStats.TimeSyncWork());

			TArray<uint8> DerivedData;

			if (GetDerivedDataCacheRef().GetSynchronous(*MeshDataKey, DerivedData, StaticMesh->GetPathName()))
			{
				COOK_STAT(Timer.AddHit(DerivedData.Num()));

				// Load from the DDC
				const bool bIsPersistent = true;
				FMemoryReader Ar(DerivedData, bIsPersistent);
				StaticMeshDescriptionBulkData->GetBulkData().Serialize(Ar, StaticMesh);

				check(GetCachedMeshDescription() == nullptr);
			}
			else
			{
				// If the DDC key doesn't exist, convert the data and save it to DDC
				// Get the RawMesh for this LOD
				FRawMesh TempRawMesh;
				RawMeshBulkData->LoadRawMesh(TempRawMesh);

				// Create a new MeshDescription
				FMeshDescription* MeshDescription = CreateMeshDescription();

				// Convert the RawMesh to MeshDescription
				TMap<int32, FName> MaterialMap;
				FillMaterialName(MaterialMap);
				FStaticMeshOperations::ConvertFromRawMesh(TempRawMesh, *MeshDescription, MaterialMap);

				// Pack MeshDescription into bulk data
				StaticMeshDescriptionBulkData->GetBulkData().SaveMeshDescription(*MeshDescription);

				// Write the DDC cache
				const bool bIsPersistent = true;
				FMemoryWriter Ar(DerivedData, bIsPersistent);
				StaticMeshDescriptionBulkData->GetBulkData().Serialize(Ar, StaticMesh);
				GetDerivedDataCacheRef().Put(*MeshDataKey, DerivedData, StaticMesh->GetPathName());
				COOK_STAT(Timer.AddMiss(DerivedData.Num()));
			}
		}

		// We now have a MeshDescription instead of a RawMesh, so get rid of the RawMesh completely
		RawMeshBulkData->Empty();
	}
}

#endif // #if WITH_EDITOR

