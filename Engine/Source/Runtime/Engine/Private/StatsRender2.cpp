// Copyright Epic Games, Inc. All Rights Reserved.

/**
 *
 * This file contains the rendering functions used in the stats code
 */

#include "Engine/Engine.h"
#include "Engine/Canvas.h"
#include "Engine/Texture2D.h"
#include "TextureResource.h"
#include "ViewportClient.h"

#if STATS

#include "Stats/StatsData.h"
#include "Performance/EnginePerformanceTargets.h"
#include "GlobalRenderResources.h"

TAutoConsoleVariable<int32> CVarNumStatsPerGroup(
	TEXT("stats.MaxPerGroup"),
	25,
	TEXT("The max number of lines of stats to show in a group")
);

/** Stats rendering constants. */
enum class EStatRenderConsts
{
	NUM_COLUMNS = 5,
};

/** Should we use a solid fill or a gradient? */
const bool bUseFlatBackgroundForStats = true;

/** Enumerates stat font types and maximum length of the stat names. */
enum class EStatFontTypes : int32
{
	/** Tiny font, used when ViewRectX < 1280. */
	Tiny = 0,
	/** Small font. */
	Small = 1,
	/** Number of fonts. */
	NumFonts,
};

/** Contains misc stat font properties. */
struct FStatFont
{
	/** Default constructor. */
	FStatFont( const int32 InMaxDisplayedChars, const int32 InFontHeight, const int32 InFontHeightOffsets ) :
		MaxDisplayedChars( InMaxDisplayedChars ),
		FontHeight( InFontHeight ),
		FontHeightOffset( InFontHeightOffsets )
	{}

	/** Maximum length of the displayed stat names. */
	const int32 MaxDisplayedChars;

	/** Font height, manually selected to fix more stats on screen. */
	const int32 FontHeight;

	/** Y offset of the background tile, manually selected to fix more stats on screen. */
	const int32 FontHeightOffset;
};

static FStatFont GStatFonts[(int32)EStatFontTypes::NumFonts] =
{
	/** Tiny. */
	FStatFont( 40, 10, 1 ),
	/** Small. */
	FStatFont( 72, 12, 2 ),
};

void LexFromString( EStatFontTypes& OutValue, const TCHAR* Buffer )
{
	OutValue = EStatFontTypes::Small;

	if( FCString::Stricmp( Buffer, TEXT( "Tiny" ) ) == 0 )
	{
		OutValue = EStatFontTypes::Tiny;
	}
}

/** Holds various parameters used for rendering stats. */
struct FStatRenderGlobals
{
	/** Rendering offset for first column from stat label. */
	int32 AfterNameColumnOffset = 0;

	/** Same as above, but for stat groups that don't need to display as many characters in the name column. */
	int32 AfterNameColumnOffsetNarrow = 0;

	/** Rendering offsets for additional columns. */
	int32 InterColumnOffset = 0;

	/** Same as above, but for stat columns that don't need to display as many value characters. */
	int32 InterColumnOffsetNarrow = 0;

	/** Color for rendering stats. */
	FLinearColor StatColor = { 0.f, 1.f, 0.f };

	/** Color to use when rendering headings. */
	FLinearColor HeadingColor = { 1.f, 0.2f, 0.f };

	/** Color to use when rendering group names. */
	FLinearColor GroupColor = FLinearColor::White;

	/** Color used as the background for every other stat item to make it easier to read across lines. */
	FLinearColor BackgroundColors[2];

	/** The font used for rendering stats. */
	UFont* StatFont = nullptr;

	/** The cached max char height of StatFont. */
	int32 StatFontHeight = 12;

	/** The legacy font size of StatFont the last time Initialize was called. */
	int32 CachedLegacyFontSize = 0;

	/** Current size of the viewport, used to detect resolution changes. */
	FIntPoint SizeXY;

	/** The X size that can be used to render the stats. */
	int32 SafeSizeX = 0;

	/** Current stat font type. */
	EStatFontTypes StatFontType = EStatFontTypes::NumFonts;

	/** Whether we are in the stereo mode. */
	bool bIsStereo = false;

	/** Whether we need update internals. */
	bool bNeedRefresh = true;

	FStatRenderGlobals()
	{
		BackgroundColors[0] = FLinearColor(0.05f, 0.05f, 0.05f, 0.92f); // dark gray mostly occluding the background
		BackgroundColors[1] = FLinearColor(0.02f, 0.02f, 0.02f, 0.88f); // slightly different to help make long lines more readable
		SetNewFont(EStatFontTypes::Small);
	}

	/**
	 * Initializes stat render globals.
	 */
	void Initialize( const int32 InSizeX, const int32 InSizeY, const int32 InSafeSizeX, const bool bInIsStereo )
	{
		FIntPoint NewSizeXY( InSizeX, InSizeY );
		if (NewSizeXY != SizeXY)
		{
			SizeXY = NewSizeXY;
			bNeedRefresh = true;
		}

		if( SizeXY.X < 1280 )
		{
			SetNewFont( EStatFontTypes::Tiny );
		}
		else
		{
			SetNewFont(EStatFontTypes::Small);
		}

		SafeSizeX = InSafeSizeX;
		bIsStereo = bInIsStereo;

		if (bIsStereo)
		{
			SetNewFont( EStatFontTypes::Tiny );
		}

		const int32 LegacyFontSize = StatFont->LegacyFontSize;
		if (LegacyFontSize != CachedLegacyFontSize)
		{
			CachedLegacyFontSize = LegacyFontSize;

			if (LegacyFontSize != Cast<UFont>(StatFont->GetClass()->GetDefaultObject())->LegacyFontSize)
			{
				StatFontHeight = StatFont->GetMaxCharHeight();
			}
			else
			{
				StatFontHeight = GStatFonts[(int32)StatFontType].FontHeight;
			}

			bNeedRefresh = true;
		}

		if( bNeedRefresh )
		{
			// This is the number of W characters to leave spacing for in the stat name column.
			const FString STATNAME_COLUMN_WIDTH = FString::ChrN( GetNumCharsForStatName(), 'S' );
			const FString STATNAME_COLUMN_WIDTH_NARROW = FString::ChrN(GetNumCharsForStatName() / 2, 'S');

			// This is the number of W characters to leave spacing for in the other stat columns.
			const FString STATDATA_COLUMN_WIDTH = FString::ChrN( StatFontType == EStatFontTypes::Small ? 8 : 7, 'W' );
			const FString STATDATA_COLUMN_WIDTH_NARROW = FString::ChrN(StatFontType == EStatFontTypes::Small ? 5 : 4, 'W');

			// Get the size of the spaces, since we can't render the width calculation strings.
			int32 StatColumnSpaceSizeY = 0;
			int32 TimeColumnSpaceSizeY = 0;

			// #TODO: Compute on the stats thread to get the exact measurement
			// Determine where the first column goes.
			StringSize(StatFont, AfterNameColumnOffset, StatColumnSpaceSizeY, *STATNAME_COLUMN_WIDTH);
			StringSize(StatFont, AfterNameColumnOffsetNarrow, StatColumnSpaceSizeY, *STATNAME_COLUMN_WIDTH_NARROW);

			// Determine the width of subsequent columns.
			StringSize(StatFont, InterColumnOffset, TimeColumnSpaceSizeY, *STATDATA_COLUMN_WIDTH);
			StringSize(StatFont, InterColumnOffsetNarrow, TimeColumnSpaceSizeY, *STATDATA_COLUMN_WIDTH_NARROW);

			bNeedRefresh = false;
		}
	}
	
	/**
	 * @return number of characters use to render the stat name
	 */
	int32 GetNumCharsForStatName() const
	{
		const int32 MaxDisplayedChars = GStatFonts[(int32)StatFontType].MaxDisplayedChars;
		return !bIsStereo ? MaxDisplayedChars : MaxDisplayedChars / 2;
	}
	
	/**
	 * @return number of characters use to render the stat name
	 */
	int32 GetFontHeight() const
	{
		return StatFontHeight;
	}

	/**
	 * @return Y offset of the background tile, so this will align with the text
	 */
	int32 GetYOffset() const
	{
		return GStatFonts[(int32)StatFontType].FontHeightOffset;
	}

	/** Sets a new font. */
	void SetNewFont( EStatFontTypes NewFontType )
	{
		if( StatFontType != NewFontType )
		{
			StatFontType = NewFontType;
			if( StatFontType == EStatFontTypes::Tiny )
			{
				StatFont = GEngine->GetTinyFont();
			}
			else if( StatFontType == EStatFontTypes::Small )
			{
				StatFont = GEngine->GetSmallFont();
			}
			bNeedRefresh = true;
		}
	}

	/** Returns the background texture for stat rows */
	FTexture* GetBackgroundTexture() const
	{
		if (bUseFlatBackgroundForStats)
		{
			return GWhiteTexture;
		}
		else
		{
			UTexture2D* BackgroundTexture = UCanvas::StaticClass()->GetDefaultObject<UCanvas>()->GradientTexture0;
			return (BackgroundTexture != nullptr) ? BackgroundTexture->GetResource() : nullptr;
		}
	}
};

FStatRenderGlobals& GetStatRenderGlobals()
{
	// Sanity checks.
	check( IsInGameThread() );
	check( GEngine );
	static FStatRenderGlobals Singleton;
	return Singleton;
}

/** Shorten a name for stats display. */
static FString ShortenName( TCHAR const* LongName )
{
	FString Result( LongName );
	const int32 Limit = GetStatRenderGlobals().GetNumCharsForStatName();
	if( Result.Len() > Limit )
	{
		Result = FString( "..." ) + Result.Right( Limit );
	}
	return Result;
}

/** Exec used to execute engine stats command on the game thread. */
static class FStatCmdEngine : private FSelfRegisteringExec
{
protected:
	/** Console commands. */
	virtual bool Exec_Runtime( UWorld*, const TCHAR* Cmd, FOutputDevice& Ar ) override
	{
		if( FParse::Command( &Cmd, TEXT( "stat" ) ) )
		{
			if( FParse::Command( &Cmd, TEXT( "display" ) ) )
			{
				TParsedValueWithDefault<EStatFontTypes> Font( Cmd, TEXT( "font=" ), EStatFontTypes::Small );
				GetStatRenderGlobals().SetNewFont( Font.Get() );
				return true;
			}
		}
		return false;
	}
}
StatCmdEngineExec;

static void RightJustify(FCanvas* Canvas, const int32 X, const int32 Y, TCHAR const* Text, FLinearColor const& Color)
{
	const FStatRenderGlobals& Globals = GetStatRenderGlobals();

	int32 StatColumnSpaceSizeX, StatColumnSpaceSizeY;
	StringSize(Globals.StatFont, StatColumnSpaceSizeX, StatColumnSpaceSizeY, Text);
	Canvas->DrawShadowedString(X + Globals.InterColumnOffset - StatColumnSpaceSizeX, Y, Text, Globals.StatFont, Color);
}


/**
 *
 * @param Item the stat to render
 * @param Canvas the render interface to draw with
 * @param X the X location to start drawing at
 * @param Y the Y location to start drawing at
 * @param Indent Indentation of this cycles, used when rendering hierarchy
 * @param bStackStat If false, this is a non-stack cycle counter, don't render the call count column
 */
static int32 RenderCycle( const FComplexStatMessage& Item, class FCanvas* Canvas, const int32 X, const int32 Y, const int32 Indent, const bool bStackStat, const float Budget, const bool bIsBudgetIgnored )
{
	const bool bBudget = Budget >= 0.f;
	const FStatRenderGlobals& Globals = GetStatRenderGlobals();
	FColor Color = Globals.StatColor.ToFColor(true);

	check(Item.NameAndInfo.GetFlag(EStatMetaFlags::IsCycle));

	const bool bIsInitialized = Item.NameAndInfo.GetField<EStatDataType>() == EStatDataType::ST_int64;

	const int32 IndentWidth = (Indent + (bIsBudgetIgnored ? 1 : 0))*8;

	if( bIsInitialized )
	{
		const float InMs = FPlatformTime::ToMilliseconds(Item.GetValue_Duration(EComplexStatField::IncAve));
		// Color will be determined by the average value of history
		// If show inclusive and and show exclusive is on, then it will choose color based on inclusive average
		// #Stats: 2015-06-09 This is slow, fix this
		FString CounterName = Item.GetShortName().ToString();
		CounterName.RemoveFromStart(TEXT("STAT_"), ESearchCase::CaseSensitive);
		GEngine->GetStatValueColoration(CounterName, InMs, Color);

		// the time of a "full bar" in ms
		const float MaxMeter = bBudget ? Budget : FEnginePerformanceTargets::GetTargetFrameTimeThresholdMS();

		const int32 MeterWidth = Globals.AfterNameColumnOffset;

		int32 BarWidth = int32((InMs / MaxMeter) * MeterWidth);
		if (BarWidth > 2)
		{
			if (BarWidth > MeterWidth ) 
			{
				BarWidth = MeterWidth;
			}

			FCanvasBoxItem BoxItem(FVector2D(X + MeterWidth - BarWidth, Y + .4f * Globals.GetFontHeight()), FVector2D(BarWidth, 0.2f * Globals.GetFontHeight()));
			BoxItem.SetColor( FLinearColor::Red );
			BoxItem.Draw( Canvas );		
		}
	}

	// #Stats: Move to the stats thread to avoid expensive computation on the game thread.
	const FString StatDesc = Item.GetDescription();
	const FString StatDisplay = StatDesc.Len() == 0 ? Item.GetShortName().GetPlainNameString() : StatDesc;

	Canvas->DrawShadowedString(X + IndentWidth, Y, *ShortenName(*StatDisplay), Globals.StatFont, Color);

	int32 CurrX = X + Globals.AfterNameColumnOffset;
	// Now append the call count
	if( bStackStat )
	{
		if (Item.NameAndInfo.GetFlag(EStatMetaFlags::IsPackedCCAndDuration) && bIsInitialized)
		{
			RightJustify(Canvas,CurrX,Y,*FString::Printf(TEXT("%u"), Item.GetValue_CallCount(EComplexStatField::IncAve)),Color);
		}
		CurrX += Globals.InterColumnOffset;
	}

	// Add the two inclusive columns if asked
	if( bIsInitialized )
	{
		RightJustify(Canvas,CurrX,Y,*FString::Printf(TEXT("%1.2f ms"),FPlatformTime::ToMilliseconds(Item.GetValue_Duration(EComplexStatField::IncAve))),Color); 
	}
	CurrX += Globals.InterColumnOffset;

	if( bIsInitialized )
	{
		RightJustify(Canvas,CurrX,Y,*FString::Printf(TEXT("%1.2f ms"),FPlatformTime::ToMilliseconds(Item.GetValue_Duration(EComplexStatField::IncMax))),Color);
		
	}
	CurrX += Globals.InterColumnOffset;

	if( bStackStat && !bBudget)
	{
		// And the exclusive if asked
		if( bIsInitialized )
		{
			RightJustify(Canvas,CurrX,Y,*FString::Printf(TEXT("%1.2f ms"),FPlatformTime::ToMilliseconds(Item.GetValue_Duration(EComplexStatField::ExcAve))),Color);
		}
		CurrX += Globals.InterColumnOffset;

		if( bIsInitialized )
		{
			RightJustify(Canvas,CurrX,Y,*FString::Printf(TEXT("%1.2f ms"),FPlatformTime::ToMilliseconds(Item.GetValue_Duration(EComplexStatField::ExcMax))),Color);
		}
		CurrX += Globals.InterColumnOffset;
	}
	return Globals.GetFontHeight();
}


static FString FormatStatValueFloat(const float Value)
{
	const float QuantizedValue = FMath::RoundToFloat(Value * 100.0f) / 100.0f;
	const float Frac = FMath::Frac(QuantizedValue);
	// #TODO: Move to stats thread, add support for int64 type, int32 may not be sufficient all the time.
	const int32 Integer = FMath::FloorToInt(QuantizedValue);
	const FString IntString = FString::FormatAsNumber(Integer);
	const FString FracString = FString::Printf(TEXT("%0.2f"), Frac);
	const FString Result = FString::Printf(TEXT("%s.%s"), *IntString, *FracString.Mid(2));
	return Result;
}

static FString FormatStatValueInt64(const int64 Value)
{
	const FString IntString = FString::FormatAsNumber((int32)Value);
	return IntString;
}


/**
 * Renders the headings for grouped rendering
 *
 * @param RI the render interface to draw with
 * @param X the X location to start rendering at
 * @param Y the Y location to start rendering at
 *
 * @return the height of headings rendered
 */
static int32 RenderGroupedHeadings( class FCanvas* Canvas, const int X, const int32 Y, const bool bIsHierarchy, const bool bBudget )
{
	// The heading looks like:
	// Stat [32chars]	CallCount [8chars]	IncAvg [8chars]	IncMax [8chars]	ExcAvg [8chars]	ExcMax [8chars]
	// If we are in budget mode ignore ExcAvg and ExcMax

	static const TCHAR* CaptionFlat = TEXT("Cycle counters (flat)");
	static const TCHAR* CaptionHier = TEXT("Cycle counters (hierarchy)");
	const FStatRenderGlobals& Globals = GetStatRenderGlobals();

	if(!bBudget)
	{
		Canvas->DrawShadowedString(X, Y, bIsHierarchy ? CaptionHier : CaptionFlat, Globals.StatFont, Globals.HeadingColor);
	}

	int32 CurrX = X + Globals.AfterNameColumnOffset;
	RightJustify(Canvas, CurrX, Y, TEXT("CallCount"), Globals.HeadingColor);
	CurrX += Globals.InterColumnOffset;

	RightJustify(Canvas, CurrX, Y, TEXT("InclusiveAvg"), Globals.HeadingColor);
	CurrX += Globals.InterColumnOffset;
	RightJustify(Canvas, CurrX, Y, TEXT("InclusiveMax"), Globals.HeadingColor);
	CurrX += Globals.InterColumnOffset;

	if(!bBudget)
	{
		RightJustify(Canvas, CurrX, Y, TEXT("ExclusiveAvg"), Globals.HeadingColor);
		CurrX += Globals.InterColumnOffset;
		RightJustify(Canvas, CurrX, Y, TEXT("ExclusiveMax"), Globals.HeadingColor);
		CurrX += Globals.InterColumnOffset;
	}
	

	return Globals.GetFontHeight() + (Globals.GetFontHeight() / 3);
}

/**
 * Renders the counter headings for grouped rendering
 *
 * @param RI the render interface to draw with
 * @param X the X location to start rendering at
 * @param Y the Y location to start rendering at
 *
 * @return the height of headings rendered
 */
static int32 RenderCounterHeadings( class FCanvas* Canvas, const int32 X, const int32 Y )
{
	// The heading looks like:
	// Stat [32chars]	Value [8chars]	Average [8chars]
	const FStatRenderGlobals& Globals = GetStatRenderGlobals();

	Canvas->DrawShadowedString(X, Y, TEXT("Counters"), Globals.StatFont, Globals.HeadingColor);

	// Determine where the first column goes
	int32 CurrX = X + Globals.AfterNameColumnOffset;

	// Draw the average column label.
	RightJustify(Canvas, CurrX, Y, TEXT("Average"), Globals.HeadingColor);
	CurrX += Globals.InterColumnOffset;

	// Draw the max column label.
	RightJustify(Canvas, CurrX, Y, TEXT("Max"), Globals.HeadingColor);
	CurrX += Globals.InterColumnOffset;

	// Draw the max column label.
	RightJustify(Canvas, CurrX, Y, TEXT("Min"), Globals.HeadingColor);

	return Globals.GetFontHeight() + (Globals.GetFontHeight() / 3);
}

/**
 * Renders the memory headings for grouped rendering
 *
 * @param RI the render interface to draw with
 * @param X the X location to start rendering at
 * @param Y the Y location to start rendering at
 *
 * @return the height of headings rendered
 */
static int32 RenderMemoryHeadings( class FCanvas* Canvas, const int32 X, const  int32 Y )
{
	// The heading looks like:
	// Stat [32chars]	MemUsed [8chars]	PhysMem [8chars]
	const FStatRenderGlobals& Globals = GetStatRenderGlobals();

	Canvas->DrawShadowedString(X, Y, TEXT("Memory Counters"), Globals.StatFont, Globals.HeadingColor);

	// Determine where the first column goes
	int32 CurrX = X + Globals.AfterNameColumnOffset;
	RightJustify(Canvas, CurrX, Y, TEXT("UsedMax"), Globals.HeadingColor);

	CurrX += Globals.InterColumnOffset;
	RightJustify(Canvas, CurrX, Y, TEXT("Mem%"), Globals.HeadingColor);

	CurrX += Globals.InterColumnOffset;
	RightJustify(Canvas, CurrX, Y, TEXT("MemPool"), Globals.HeadingColor);

	CurrX += Globals.InterColumnOffset;
	RightJustify(Canvas, CurrX, Y, TEXT("Pool Capacity"), Globals.HeadingColor);

	return Globals.GetFontHeight() + (Globals.GetFontHeight() / 3);
}

static int32 RenderMemoryCounter( const FGameThreadStatsData& ViewData, const FComplexStatMessage& All, class FCanvas* Canvas, const int32 X, const int32 Y, const float Budget, const bool bIsBudgetIgnored )
{
	FPlatformMemory::EMemoryCounterRegion Region = FPlatformMemory::EMemoryCounterRegion(All.NameAndInfo.GetField<EMemoryRegion>());
	// At this moment we only have memory stats that are marked as non frame stats, so can't be cleared every frame.
	//const bool bDisplayAll = All.NameAndInfo.GetFlag(EStatMetaFlags::ShouldClearEveryFrame);
	const FStatRenderGlobals& Globals = GetStatRenderGlobals();
	const float MaxMemUsed = All.GetValue_double(EComplexStatField::IncMax);

	// Draw the label
	Canvas->DrawShadowedString(X, Y, *ShortenName(*All.GetDescription()), Globals.StatFont, Globals.StatColor);
	int32 CurrX = X + Globals.AfterNameColumnOffset;

	// always use MB for easier comparisons
	const bool bAutoType = false;

	// Now append the max value of the stat
	RightJustify(Canvas, CurrX, Y, *GetMemoryString(MaxMemUsed, bAutoType), Globals.StatColor);
	CurrX += Globals.InterColumnOffset;
	if (ViewData.PoolCapacity.Contains(Region))
	{
		RightJustify(Canvas, CurrX, Y, *FString::Printf(TEXT("%.0f%%"), float(100.0 * MaxMemUsed / double(ViewData.PoolCapacity[Region]))), Globals.StatColor);
	}
	CurrX += Globals.InterColumnOffset;
	if (ViewData.PoolAbbreviation.Contains(Region))
	{
		RightJustify(Canvas, CurrX, Y, *ViewData.PoolAbbreviation[Region], Globals.StatColor);
	}
	CurrX += Globals.InterColumnOffset;
	if (ViewData.PoolCapacity.Contains(Region))
	{
		RightJustify(Canvas, CurrX, Y, *GetMemoryString(double(ViewData.PoolCapacity[Region]), bAutoType), Globals.StatColor);
	}
	CurrX += Globals.InterColumnOffset;

	return Globals.GetFontHeight();
}

static void RenderCounterLabel(const FComplexStatMessage& All, class FCanvas* Canvas, const int32 X, const int32 Y)
{
	const FStatRenderGlobals& Globals = GetStatRenderGlobals();

	const FString StatDesc = All.GetDescription();
	const FString StatDisplay = StatDesc.Len() == 0 ? All.GetShortName().GetPlainNameString() : StatDesc;
	Canvas->DrawShadowedString(X, Y, *ShortenName(*StatDisplay), Globals.StatFont, Globals.StatColor);
}

static void RenderCounterValues(const FComplexStatMessage& All, class FCanvas* Canvas, int32& X, const int32 Y, int32 Spacing, float ZeroStatAlphaMult = 1.0f)
{
	const FStatRenderGlobals& Globals = GetStatRenderGlobals();

	auto RenderValue = [&](EComplexStatField::Type Field)
	{
		if (All.NameAndInfo.GetField<EStatDataType>() == EStatDataType::ST_double)
		{
			const FString ValueFormatted = FormatStatValueFloat(All.GetValue_double(Field));

			FLinearColor Color = Globals.StatColor;
			if (ValueFormatted == TEXT("0.00"))
			{
				Color.A *= ZeroStatAlphaMult;
			}

			RightJustify(Canvas, X, Y, *ValueFormatted, Color);
		}
		else if (All.NameAndInfo.GetField<EStatDataType>() == EStatDataType::ST_int64)
		{
			int64 Value = All.GetValue_int64(Field);
			const FString ValueFormatted = FormatStatValueInt64(Value);

			FLinearColor Color = Globals.StatColor;
			if (Value == 0)
			{
				Color.A *= ZeroStatAlphaMult;
			}

			RightJustify(Canvas, X, Y, *ValueFormatted, Color);
		}
	};

	if (All.NameAndInfo.GetFlag(EStatMetaFlags::ShouldClearEveryFrame))
	{
		// Append the average.
		RenderValue(EComplexStatField::IncAve);
	}
	X += Spacing;

	// Append the maximum.
	RenderValue(EComplexStatField::IncMax);
	X += Spacing;

	// Append the minimum.
	RenderValue(EComplexStatField::IncMin);
	X += Spacing;
}

static int32 RenderCounter( const FGameThreadStatsData& ViewData, const FComplexStatMessage& All, class FCanvas* Canvas, const int32 X, const int32 Y, const float Budget, const bool bIsBudgetIgnored )
{
	const FStatRenderGlobals& Globals = GetStatRenderGlobals();

	// If this is a cycle, render it as a cycle. This is a special case for manually set cycle counters.
	const bool bIsCycle = All.NameAndInfo.GetFlag(EStatMetaFlags::IsCycle);
	if( bIsCycle )
	{
		return RenderCycle( All, Canvas, X, Y, 0, false, Budget, bIsBudgetIgnored );
	}

	// Draw the label
	int32 CurrX = X;
	RenderCounterLabel(All, Canvas, CurrX, Y);
	CurrX += Globals.AfterNameColumnOffset;
	RenderCounterValues(All, Canvas, CurrX, Y, Globals.InterColumnOffset);

	return Globals.GetFontHeight();
}

void RenderHierCycles( FCanvas* Canvas, const int32 X, int32& Y, const FActiveStatGroupInfo& HudGroup )
{
	const FStatRenderGlobals& Globals = GetStatRenderGlobals();
	const FTexture* BackgroundTexture = Globals.GetBackgroundTexture();

	// Render all cycle counters.
	for( int32 RowIndex = 0; RowIndex < HudGroup.HierAggregate.Num(); ++RowIndex )
	{
		const FComplexStatMessage& ComplexStat = HudGroup.HierAggregate[RowIndex];
		const int32 Indent = HudGroup.Indentation[RowIndex];

		if (BackgroundTexture != nullptr)
		{
			Canvas->DrawTile(X, Y + Globals.GetYOffset(), Globals.AfterNameColumnOffset + Globals.InterColumnOffset * (int32)EStatRenderConsts::NUM_COLUMNS, Globals.GetFontHeight(),
				0, 0, 1, 1,
				Globals.BackgroundColors[RowIndex & 1], BackgroundTexture, true);
		}

		Y += RenderCycle( ComplexStat, Canvas, X, Y, Indent, true, -1.f, false );
	}
}


int32 RenderGroupBudget( FCanvas* Canvas, const  int32 X, const int32 Y, const uint64 AvgTotalTime, const  uint64 MaxTotalTime, const float GroupBudget )
{
	// The budget looks like:
	// Stat [32chars]	Value [8chars]	Average [8chars]
	const FStatRenderGlobals& Globals = GetStatRenderGlobals();

	const float AvgTotalMs = FPlatformTime::ToMilliseconds(AvgTotalTime);
	const float MaxTotalMs = FPlatformTime::ToMilliseconds(MaxTotalTime);

	FString BudgetString = FString::Printf(TEXT("Total (of %1.2f ms)"), GroupBudget);

	Canvas->DrawShadowedString(X, Y, *BudgetString, Globals.StatFont, FLinearColor::Green);

	int32 CurrX = X + Globals.AfterNameColumnOffset;
	CurrX += Globals.InterColumnOffset;
   
	RightJustify(Canvas, CurrX, Y, *FString::Printf(TEXT("%1.2f ms"), AvgTotalMs) , AvgTotalMs > GroupBudget ? FLinearColor::Red : FLinearColor::Green);
	
	CurrX += Globals.InterColumnOffset;
	RightJustify(Canvas, CurrX, Y, *FString::Printf(TEXT("%1.2f ms"), MaxTotalMs) , MaxTotalMs > GroupBudget ? FLinearColor::Red : FLinearColor::Green);
	
	return Globals.GetFontHeight();
}

int32 RenderMoreStatsLine(FCanvas* Canvas, const  int32 X, const int32 Y, const int32 NumMoreStats)
{
	const FStatRenderGlobals& Globals = GetStatRenderGlobals();

	FString MoreString = FString::Printf(TEXT("[%d more stats. Use the stats.MaxPerGroup CVar to increase the limit]"), NumMoreStats);
	Canvas->DrawShadowedString(X, Y, *MoreString, Globals.StatFont, FLinearColor::Yellow);

	return Globals.GetFontHeight();
}

template< typename T >
void RenderArrayOfStats( FCanvas* Canvas, const int32 X, int32& Y, const TArray<FComplexStatMessage>& Aggregates, const FGameThreadStatsData& ViewData, const TSet<FName>& IgnoreBudgetStats, const float TotalGroupBudget, const T& FunctionToCall )
{
	const FStatRenderGlobals& Globals = GetStatRenderGlobals();
	const FTexture* BackgroundTexture = Globals.GetBackgroundTexture();

	const bool bBudget = TotalGroupBudget >= 0.f;
	const int32 NumColumns = (int32)EStatRenderConsts::NUM_COLUMNS - (bBudget ? 2 : 0);
	uint64 AvgTotalTime = 0;
	uint64 MaxTotalTime = 0;

	// Render all counters.
	int32 MaxStatsPerGroup = CVarNumStatsPerGroup.GetValueOnGameThread();
	int32 RowIndex;
	for( RowIndex = 0; RowIndex < Aggregates.Num() && RowIndex < MaxStatsPerGroup; ++RowIndex )
	{
		const FComplexStatMessage& ComplexStat = Aggregates[RowIndex];
		const bool bIsBudgetIgnored = IgnoreBudgetStats.Contains(ComplexStat.NameAndInfo.GetShortName());
		if(bBudget && !bIsBudgetIgnored && ComplexStat.NameAndInfo.GetFlag(EStatMetaFlags::IsPackedCCAndDuration))
		{
			AvgTotalTime += ComplexStat.GetValue_Duration(EComplexStatField::IncAve);
			MaxTotalTime += ComplexStat.GetValue_Duration(EComplexStatField::IncMax);
		}

		if (BackgroundTexture != nullptr)
		{
			Canvas->DrawTile(X, Y + Globals.GetYOffset(), Globals.AfterNameColumnOffset + (Globals.InterColumnOffset) * NumColumns, Globals.GetFontHeight(),
				0, 0, 1, 1,
				Globals.BackgroundColors[RowIndex & 1], BackgroundTexture, true);
		}

		Y += FunctionToCall( ViewData, ComplexStat, Canvas, X, Y, TotalGroupBudget, bIsBudgetIgnored );

		// Stop drawing after we have gone off the bottom of the screen, if in budget mode continue to gather avg/max stats
		if (!bBudget && Y > (Globals.SizeXY.Y + Globals.GetFontHeight()))
		{
			break;
		}
	}

	if (MaxStatsPerGroup < Aggregates.Num())
	{
		Canvas->DrawTile(X, Y + Globals.GetYOffset(), Globals.AfterNameColumnOffset + Globals.InterColumnOffset * NumColumns, Globals.GetFontHeight(),
			0, 0, 1, 1,
			Globals.BackgroundColors[(RowIndex++) & 1], BackgroundTexture, true);
		Y += RenderMoreStatsLine(Canvas, X, Y, Aggregates.Num() - MaxStatsPerGroup);
	}

	if(bBudget)
	{
		Canvas->DrawTile(X, Y + Globals.GetYOffset(), Globals.AfterNameColumnOffset + Globals.InterColumnOffset * NumColumns, Globals.GetFontHeight(),
			0, 0, 1, 1,
			Globals.BackgroundColors[RowIndex & 1], BackgroundTexture, true);
		Y += RenderGroupBudget(Canvas, X, Y, AvgTotalTime, MaxTotalTime, TotalGroupBudget);
	}
}

struct FGpuStatMessage
{
	const FComplexStatMessage* Busy;
	const FComplexStatMessage* Wait;
	const FComplexStatMessage* Idle;
};

static int32 RenderGpuStatHeadings(class FCanvas* Canvas, const int32 X, const int32 Y)
{
	const FStatRenderGlobals& Globals = GetStatRenderGlobals();

	Canvas->DrawShadowedString(X, Y, TEXT("Counters"), Globals.StatFont, Globals.HeadingColor);

	TCHAR const* Types[] =
	{
		TEXT("Busy"),
		TEXT("Wait"),
		TEXT("Idle")
	};

	// Draw headers
	int32 CurrX = X + Globals.AfterNameColumnOffsetNarrow + Globals.InterColumnOffsetNarrow;
	for (TCHAR const* Type : Types)
	{
		RightJustify(Canvas, CurrX, Y - Globals.GetFontHeight(), Type, Globals.HeadingColor);
		CurrX += Globals.InterColumnOffsetNarrow * 2 + Globals.InterColumnOffset;
	}

	// Draw column labels
	CurrX = X + Globals.AfterNameColumnOffsetNarrow;
	for (TCHAR const* Type : Types)
	{
		// Draw the average column label.
		RightJustify(Canvas, CurrX, Y, TEXT("Avg"), Globals.HeadingColor);
		CurrX += Globals.InterColumnOffsetNarrow;

		// Draw the max column label.
		RightJustify(Canvas, CurrX, Y, TEXT("Max"), Globals.HeadingColor);
		CurrX += Globals.InterColumnOffsetNarrow;

		// Draw the max column label.
		RightJustify(Canvas, CurrX, Y, TEXT("Min"), Globals.HeadingColor);

		// Use a larger space between the stat types
		CurrX += Globals.InterColumnOffset;
	}

	return Globals.GetFontHeight() + (Globals.GetFontHeight() / 3);
}

void RenderArrayOfGpuStats(FCanvas* Canvas, const int32 X, int32& Y, const TMap<FName, FGpuStatMessage>& Aggregates, const FGameThreadStatsData& ViewData)
{
	const FStatRenderGlobals& Globals = GetStatRenderGlobals();
	const FTexture* BackgroundTexture = Globals.GetBackgroundTexture();

	int32 RowIndex = 0;
	auto DrawRowBackground = [&]()
	{
		if (BackgroundTexture != nullptr)
		{
			Canvas->DrawTile(
				X,
				Y + Globals.GetYOffset(),
				Globals.AfterNameColumnOffsetNarrow + Globals.InterColumnOffsetNarrow * 6 + Globals.InterColumnOffset * 3,
				Globals.GetFontHeight(),
				0, 0, 1, 1,
				Globals.BackgroundColors[RowIndex & 1],
				BackgroundTexture,
				true
			);
		}
	};

	// Render all counters.
	int32 MaxStatsPerGroup = CVarNumStatsPerGroup.GetValueOnGameThread();
	for (auto const& [Name, Stat] : Aggregates)
	{
		if (RowIndex >= MaxStatsPerGroup)
		{
			break;
		}
		
		DrawRowBackground();

		int32 CurrX = X;
		const FComplexStatMessage* StatForLabel = 
			Stat.Busy ? Stat.Busy :
			Stat.Wait ? Stat.Wait : 
			Stat.Idle ? Stat.Idle : nullptr;

		if (StatForLabel)
		{
			RenderCounterLabel(*StatForLabel, Canvas, CurrX, Y);
			CurrX += Globals.AfterNameColumnOffsetNarrow;

			// Render the Busy, Wait and Idle stats all on the same row
			auto RenderCol = [&](const FComplexStatMessage* Message)
			{
				if (Message)
				{
					RenderCounterValues(*Message, Canvas, CurrX, Y, Globals.InterColumnOffsetNarrow, 0.4f);
				}
				else
				{
					CurrX += Globals.InterColumnOffsetNarrow * 3;
				}

				// Adjust for a wider space between stat types
				CurrX -= Globals.InterColumnOffsetNarrow;
				CurrX += Globals.InterColumnOffset;
			};

			RenderCol(Stat.Busy);
			RenderCol(Stat.Wait);
			RenderCol(Stat.Idle);

			Y += Globals.GetFontHeight();
		}

		// Stop drawing after we have gone off the bottom of the screen
		if (Y > (Globals.SizeXY.Y + Globals.GetFontHeight()))
		{
			break;
		}

		RowIndex++;
	}

	if (MaxStatsPerGroup < Aggregates.Num())
	{
		DrawRowBackground();
		Y += RenderMoreStatsLine(Canvas, X, Y, Aggregates.Num() - MaxStatsPerGroup);
	}
}

static int32 RenderFlatCycle( const FGameThreadStatsData& ViewData, const FComplexStatMessage& Item, class FCanvas* Canvas, const int32 X, const int32 Y, const  float Budget, const bool bIsBudgetIgnored )
{
	return RenderCycle( Item, Canvas, X, Y, 0, true, Budget, bIsBudgetIgnored);
}

/**
 * Renders stats using groups
 *
 * @param ViewData data from the stats thread
 * @param RI the render interface to draw with
 * @param X the X location to start rendering at
 * @param Y the Y location to start rendering at
 */
static void RenderGroupedWithHierarchy(const FGameThreadStatsData& ViewData, FViewport* Viewport, class FCanvas* Canvas, const int32 X, int32& Y)
{
	// Grab texture for rendering text background.
	UTexture2D* BackgroundTexture = UCanvas::StaticClass()->GetDefaultObject<UCanvas>()->DefaultTexture;

	const FStatRenderGlobals& Globals = GetStatRenderGlobals();

	// Render all groups.
	for( int32 GroupIndex = 0; GroupIndex < ViewData.ActiveStatGroups.Num(); ++GroupIndex )
	{
		const FActiveStatGroupInfo& StatGroup = ViewData.ActiveStatGroups[GroupIndex];

		// If the stat isn't enabled for this particular viewport, skip
		FString StatGroupName = ViewData.GroupNames[GroupIndex].ToString();
		StatGroupName.RemoveFromStart(TEXT("STATGROUP_"), ESearchCase::CaseSensitive);
		if (!Viewport->GetClient() || !Viewport->GetClient()->IsStatEnabled(StatGroupName))
		{
			continue;
		}

		const bool bBudget = StatGroup.ThreadBudgetMap.Num() > 0;
		const int32 NumThreadsBreakdown = bBudget ? StatGroup.FlatAggregateThreadBreakdown.Num() : 1;
		TArray<FName> ThreadNames;
		StatGroup.FlatAggregateThreadBreakdown.GetKeys(ThreadNames);

		for(int32 ThreadBreakdownIdx = 0; ThreadBreakdownIdx < NumThreadsBreakdown; ++ThreadBreakdownIdx)
		{
			// Render header.
			const FName& GroupName = ViewData.GroupNames[GroupIndex];
			const FString& GroupDesc = ViewData.GroupDescriptions[GroupIndex];
			FString GroupLongName = FString::Printf(TEXT("%s [%s]"), *GroupDesc, *GroupName.GetPlainNameString());
			
			FName ThreadName;
			FName ShortThreadName;
			if(bBudget)
			{
				ThreadName = ThreadNames[ThreadBreakdownIdx];
				ShortThreadName = FStatNameAndInfo::GetShortNameFrom(ThreadName);
				GroupLongName += FString::Printf(TEXT(" - %s"), *ShortThreadName.ToString());
			}

			if(!ViewData.RootFilter.IsEmpty())
			{
				GroupLongName += FString::Printf(TEXT(" ROOT=%s"), *ViewData.RootFilter);
			}
			Canvas->DrawShadowedString(X, Y, *GroupLongName, Globals.StatFont, Globals.GroupColor);
			Y += Globals.GetFontHeight();

			
			const bool bHasHierarchy = !!StatGroup.HierAggregate.Num();
			const bool bHasFlat = !!StatGroup.FlatAggregate.Num();

			if (bHasHierarchy || bHasFlat)
			{
				// Render grouped headings.
				Y += RenderGroupedHeadings(Canvas, X, Y, bHasHierarchy, bBudget);
			}

			// Render hierarchy.
			if (bHasHierarchy)
			{
				RenderHierCycles(Canvas, X, Y, StatGroup);
				Y += Globals.GetFontHeight();
			}

			const float* BudgetPtr = ShortThreadName != NAME_None ? StatGroup.ThreadBudgetMap.Find(ShortThreadName) : nullptr;
			const float Budget = BudgetPtr ? *BudgetPtr : -1.f;
			// Render flat.
			if (bHasFlat)
			{
				RenderArrayOfStats(Canvas, X, Y, bBudget ? StatGroup.FlatAggregateThreadBreakdown[ThreadName] : StatGroup.FlatAggregate, ViewData, StatGroup.BudgetIgnoreStats, Budget, RenderFlatCycle);
				Y += Globals.GetFontHeight();
			}
		}
		

		// Render memory counters.
		if (StatGroup.MemoryAggregate.Num())
		{
			Y += RenderMemoryHeadings(Canvas, X, Y);
			RenderArrayOfStats(Canvas, X, Y, StatGroup.MemoryAggregate, ViewData, StatGroup.BudgetIgnoreStats, -1.f, RenderMemoryCounter);
			Y += Globals.GetFontHeight();
		}

		// Render remaining counters.
		if (StatGroup.CountersAggregate.Num())
		{
			Y += RenderCounterHeadings(Canvas, X, Y);
			RenderArrayOfStats(Canvas, X, Y, StatGroup.CountersAggregate, ViewData, StatGroup.BudgetIgnoreStats, -1.f, RenderCounter);
			Y += Globals.GetFontHeight();
		}

		// Render GPU stats
		if (StatGroup.GpuStatsAggregate.Num())
		{
			// Group together the individual GPU Busy, Wait and Idle stats 
			TMap<FName, FGpuStatMessage> GroupedStats;
			for (const FComplexStatMessage& Message : StatGroup.GpuStatsAggregate)
			{
				FName ShortName = Message.GetShortName();
				int32 Number = ShortName.GetNumber();
				ShortName.SetNumber(0);

				using EType = UE::RHI::GPUProfiler::FGPUStat::EType;

				FGpuStatMessage& Stat = GroupedStats.FindOrAdd(ShortName);
				switch (EType(Number))
				{
				default: checkNoEntry(); [[fallthrough]];
				case EType::Busy: Stat.Busy = &Message; break;
				case EType::Wait: Stat.Wait = &Message; break;
				case EType::Idle: Stat.Idle = &Message; break;
				}
			}

			Y += RenderGpuStatHeadings(Canvas, X, Y);
			RenderArrayOfGpuStats(Canvas, X, Y, GroupedStats, ViewData);
			Y += Globals.GetFontHeight();
		}
	}
}

/**
 * Renders the stats data
 *
 * @param Viewport	The viewport to render to
 * @param Canvas	Canvas object to use for rendering
 * @param X			the X location to start rendering at
 * @param Y			the Y location to start rendering at
 * @param SafeSizeX	the X size that can be used to render the stats
 */
void RenderStats(FViewport* Viewport, class FCanvas* Canvas, int32 X, int32 Y, int32 SafeSizeX)
{
	DECLARE_SCOPE_CYCLE_COUNTER( TEXT( "RenderStats" ), STAT_RenderStats, STATGROUP_StatSystem );

	FGameThreadStatsData* ViewData = FLatestGameThreadStatsData::Get().Latest;
	if (!ViewData || !ViewData->bRenderStats)
	{
		return;
	}
	
	FStatRenderGlobals& Globals = GetStatRenderGlobals();
	// SizeX is used to clip/arrange the rendered stats to avoid overlay in stereo mode.
	const bool bIsStereo = Canvas->IsStereoRendering();
	Globals.Initialize( Viewport->GetSizeXY().X/Canvas->GetDPIScale(), Viewport->GetSizeXY().Y/Canvas->GetDPIScale(), SafeSizeX, bIsStereo );

	if( !ViewData->bDrawOnlyRawStats )
	{
		RenderGroupedWithHierarchy(*ViewData, Viewport, Canvas, X, Y);
	}
	else
	{
		// Render all counters.
		for( int32 RowIndex = 0; RowIndex < ViewData->GroupDescriptions.Num(); ++RowIndex )
		{
			Canvas->DrawShadowedString(X, Y, *ViewData->GroupDescriptions[RowIndex], Globals.StatFont, Globals.StatColor);
			Y += Globals.GetFontHeight();
		}
	}
}

#endif