// Copyright Epic Games, Inc. All Rights Reserved.

#include "StereoLayerAdditionalFlagsManager.h"

#include "IStereoLayers.h"
#include "IStereoLayersFlagsSupplier.h"
#include "Features/IModularFeatures.h"

DEFINE_LOG_CATEGORY(LogStereoLayerFlags);

TSharedPtr<FStereoLayerAdditionalFlagsManager> FStereoLayerAdditionalFlagsManager::Instance = NULL;

TSharedPtr<FStereoLayerAdditionalFlagsManager> FStereoLayerAdditionalFlagsManager::Get()
{
	if (!Instance.IsValid())
	{
		Instance = MakeShared<FStereoLayerAdditionalFlagsManager>();
		Instance->CollectFlags();
		Instance->CreateRuntimeFlagsMap();
	}

	return Instance;
}

void FStereoLayerAdditionalFlagsManager::Destroy()
{
	if (Instance.IsValid())
	{
		Instance.Reset();
	}
}

void FStereoLayerAdditionalFlagsManager::CollectFlags(TSet<FName>& OutFlags)
{
	TArray<IStereoLayersFlagsSupplier*> FlagsSuppliers = IModularFeatures::Get().GetModularFeatureImplementations<IStereoLayersFlagsSupplier>(IStereoLayersFlagsSupplier::GetModularFeatureName());
	for (IStereoLayersFlagsSupplier* FlagSupplier : FlagsSuppliers)
	{
		if (FlagSupplier)
		{
			FlagSupplier->EnumerateFlags(OutFlags);
		}
	}
}

void FStereoLayerAdditionalFlagsManager::CollectFlags()
{
	CollectFlags(UniqueFlags);
}

void FStereoLayerAdditionalFlagsManager::CreateRuntimeFlagsMap()
{
	RuntimeFlags.Empty();
	uint32 Value = IStereoLayers::ELayerFlags::LAYER_FLAG_MAX_VALUE << 1;
	for (FName& Flag : UniqueFlags)
	{
		RuntimeFlags.Add(Flag, Value);
		Value = Value << 1;
	}
}

uint32 FStereoLayerAdditionalFlagsManager::GetFlagValue(const FName Flag) const
{
	const uint32* Value = RuntimeFlags.Find(Flag);
	if (Value)
	{
		return *Value;
	}
	UE_LOGF(LogStereoLayerFlags, Log, "Flag %ls was not found.", *(Flag.ToString()));
	return 0;
}