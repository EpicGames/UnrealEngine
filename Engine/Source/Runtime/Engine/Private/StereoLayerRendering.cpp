// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	ScreenRendering.cpp: Screen rendering implementation.
=============================================================================*/

#include "StereoLayerRendering.h"
#include "Misc/DelayedAutoRegister.h"

IMPLEMENT_TYPE_LAYOUT(FStereoLayerPS_Base);
IMPLEMENT_SHADER_TYPE(,FStereoLayerVS,TEXT("/Engine/Private/StereoLayerShader.usf"),TEXT("MainVS"),SF_Vertex);
IMPLEMENT_SHADER_TYPE(,FStereoLayerPS,TEXT("/Engine/Private/StereoLayerShader.usf"),TEXT("MainPS_Texture2D"),SF_Pixel);
IMPLEMENT_SHADER_TYPE(,FStereoLayerPS_External,TEXT("/Engine/Private/StereoLayerShader.usf"),TEXT("MainPS_TextureExternal"),SF_Pixel);
