// Copyright Epic Games, Inc. All Rights Reserved.

#include "StereoLayerShapes.h"

const FName FQuadLayer::ShapeName = FName("QuadLayer");
const FName FCylinderLayer::ShapeName = FName("CylinderLayer");
const FName FCubemapLayer::ShapeName = FName("CubemapLayer");
const FName FEquirectLayer::ShapeName = FName("EquirectLayer");
