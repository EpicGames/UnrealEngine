// Copyright Epic Games, Inc. All Rights Reserved.

#include "StereoRendering.h"
#include "SceneView.h"
#include "GeneralProjectSettings.h"
#include "RenderGraphEvent.h"
#include "RenderGraphUtils.h"
#include "Misc/CommandLine.h"
#include "Misc/ConfigCacheIni.h"

static TAutoConsoleVariable<int32> CVarForceStartInVR(
	TEXT("r.ForceStartInVR"),
	0,
	TEXT("If true, the game will attempt to start in VR, regardless of whether \"Start in VR\" is true in the project settings or -vr was set on the commandline."),
	ECVF_ReadOnly);

bool IStereoRendering::IsStereoEyeView(const FSceneView& View)
{
	return IsStereoEyePass(View.StereoPass);
}

bool IStereoRendering::IsAPrimaryView(const FSceneView& View)
{
	return IsAPrimaryPass(View.StereoPass);
}

bool IStereoRendering::IsASecondaryView(const FSceneView& View)
{
	return IsASecondaryPass(View.StereoPass);
}

BEGIN_SHADER_PARAMETER_STRUCT(FRenderTextureDeprecationPass,)
	RDG_TEXTURE_ACCESS(SrcTexture, ERHIAccess::SRVGraphics)
	RENDER_TARGET_BINDING_SLOTS()
END_SHADER_PARAMETER_STRUCT()

void IStereoRendering::RenderTexture_RenderThread(FRDGBuilder& GraphBuilder, FRDGTextureRef BackBuffer, FRDGTextureRef SrcTexture, FVector2f WindowSize) const
{
	// If this is not defined by an implementation of IStereoRendering, instead fall back to wrapping the deprecated older RHICmdList version in a pass
	// This may cause redundant transitions if transitions are still performed manually in the RHICmdList version
	FRenderTextureDeprecationPass* Pass = GraphBuilder.AllocParameters<FRenderTextureDeprecationPass>();
	Pass->SrcTexture = SrcTexture;
	Pass->RenderTargets[0] = FRenderTargetBinding(BackBuffer, ERenderTargetLoadAction::ELoad);
	GraphBuilder.AddPass(RDG_EVENT_NAME("IStereoRendering_RenderTexture_DeprecationStub"), Pass, ERDGPassFlags::Raster, [this, Pass, WindowSize](FRHICommandListImmediate& RHICmdList)
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		RenderTexture_RenderThread(RHICmdList,
			Pass->RenderTargets[0].GetTexture() ? Pass->RenderTargets[0].GetTexture()->GetRHI() : nullptr,
			Pass->SrcTexture.GetTexture() ? Pass->SrcTexture.GetTexture()->GetRHI() : nullptr,
			FVector2D(WindowSize));
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	});
}

bool IStereoRendering::IsStartInVR()
{
	bool bStartInVR = false;

	if (CVarForceStartInVR.GetValueOnAnyThread() > 0)
	{
		bStartInVR = true;
	}
	else if (IsClassLoaded<UGeneralProjectSettings>())
	{
		bStartInVR = GetDefault<UGeneralProjectSettings>()->bStartInVR;
	}
	else
	{
		GConfig->GetBool(TEXT("/Script/EngineSettings.GeneralProjectSettings"), TEXT("bStartInVR"), bStartInVR, GGameIni);
	}

	return FParse::Param(FCommandLine::Get(), TEXT("vr")) || bStartInVR;
}
