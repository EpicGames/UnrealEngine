// Copyright Epic Games, Inc. All Rights Reserved.

#include "StreamableAsyncLoader.h"
#include "Logging/LogMacros.h"
#include "Async/UniqueLock.h"
#include "Engine/StreamableManager.h"
#include "HAL/IConsoleManager.h"
#include "ProfilingDebugging/StreamableManagerProfiling.h"

DEFINE_LOG_CATEGORY_STATIC(LogStreamableAsyncLoading, Log, All);

static float GStreamableJITAsyncLoadingInitialBatchingFactor = 0.25f;
static FAutoConsoleVariableRef CVarStreamableJITAsyncLoadingInitialBatchingFactor(
	TEXT("s.StreamableJITAsyncLoadingInitialBatchingFactor"),
	GStreamableJITAsyncLoadingInitialBatchingFactor,
	TEXT("Number of requests pushed initially to the async load queue (= Factor * NumTotalQueued)"),
	ECVF_Default
);

void FStreamableAsyncLoader::AddAsyncLoadRequest(const FPackagePath& InPackagePath, FStreamableAsyncLoaderRequestQueuedDelegate&& InRequestQueuedDelegate, FLoadPackageAsyncDelegate&& InCompletionDelegate, TAsyncLoadPriority InPriority)
{
	int32 RequestID = LoadPackageAsync(InPackagePath,
		NAME_None /* PackageNameToCreate */,
		MoveTemp(InCompletionDelegate) /* InCompletionDelegate */,
		PKG_None /* InPackageFlags */,
		INDEX_NONE /* InPIEInstanceID */,
		InPriority /* InPackagePriority */);
	InRequestQueuedDelegate.ExecuteIfBound(RequestID);
	UE_LOGF(LogStreamableAsyncLoading, Verbose, "FStreamableAsyncLoader queued async load request [Package:%ls] [Priority: %d] [RequestID]:%d", *InPackagePath.GetDebugName(), InPriority, RequestID);
}

void FStreamableAsyncLoader::StartAsyncLoadRequests()
{
	// Do nothing. Requests were already queued above.
}

FStreamableJustInTimeAsyncLoader::FStreamableJustInTimeAsyncLoader(TWeakPtr<FStreamableHandle> InHandle)
	: OwningHandle(InHandle)
{
	// The async loader is created on the game thread when the request is queued.
	check(IsInGameThread());

#if UE_STREAMABLE_MANAGER_PROFILING_ENABLED
	FStreamableManagerProfiling::OnJITLoaderCreated();	
#endif // UE_STREAMABLE_MANAGER_PROFILING_ENABLED
}

FStreamableJustInTimeAsyncLoader::~FStreamableJustInTimeAsyncLoader()
{
	// The async loader can be destroyed on either the game thread or the async loading thread depending on ownership.
	// Ownership is transferred from the game thread to the async loading thread once requests are queued to the async loading thread.
	check(IsInGameThread() || IsInAsyncLoadingThread());
	
#if UE_STREAMABLE_MANAGER_PROFILING_ENABLED
	FStreamableManagerProfiling::OnJITLoaderDestroyed();
#endif // UE_STREAMABLE_MANAGER_PROFILING_ENABLED
}

void FStreamableJustInTimeAsyncLoader::AddAsyncLoadRequest(const FPackagePath& InPackagePath, FStreamableAsyncLoaderRequestQueuedDelegate&& InRequestQueuedDelegate, FLoadPackageAsyncDelegate&& InCompletionDelegate, TAsyncLoadPriority InPriority)
{
	// This is expected to be called on the game thread when setting up the async loader with requests.
	check(IsInGameThread());
	
	// InPriority is ignored here and queried from the handle when queueing requests.
	AsyncLoadRequests.Enqueue(InPackagePath, MoveTemp(InRequestQueuedDelegate), MoveTemp(InCompletionDelegate));
	TotalRemainingRequests.fetch_add(1, std::memory_order_relaxed);
}

void FStreamableJustInTimeAsyncLoader::StartAsyncLoadRequests()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FStreamableJustInTimeAsyncLoader::StartAsyncLoadRequests);

	// This is expected to be called on the game thread when starting to queue the initial batch of requests to the async load queue.
	check(IsInGameThread());

	if(AsyncLoadRequests.IsEmpty())
	{
		return;
	}

	UE_CLOGF(OwningHandle.IsValid(), LogStreamableAsyncLoading, Verbose, "FStreamableJustInTimeAsyncLoader [%ls] started processing async load requests", *OwningHandle.Pin()->GetDebugName());

	check(!ProgressDelegate.IsValid());
	if (!ProgressDelegate.IsValid())
	{
		// Keep the loader instance alive by passing the progress delegate to the async loader until eventually cleaned up (see FStreamableJustInTimeAsyncLoader::Cleanup).
		TSharedRef<IStreamableAsyncLoader> SharedThisRef = AsShared();
		ProgressDelegate = MakeShared<FLoadPackageAsyncProgressDelegate>(FLoadPackageAsyncProgressDelegate::DelegateType::CreateLambda([SharedThisRef](FLoadPackageAsyncProgressParams& InParams)
		{
			// Can be called on game thread (for EAsyncLoadingProgress::FullyLoaded) or async loading thread (for EAsyncLoadingProgress::Serialized && EAsyncLoadingProgress::Failed).
			static_cast<FStreamableJustInTimeAsyncLoader&>(SharedThisRef.Get()).MakeProgress(InParams);
		}),
		// Only listen to Serialized, Fully Loaded, or Failed progress to chain requests.
		FLoadPackageAsyncProgressDelegate::BuildMask(EAsyncLoadingProgress::Serialized, EAsyncLoadingProgress::FullyLoaded, EAsyncLoadingProgress::Failed));
	}

	UpdateHandleState();

	const float ToBatch = GStreamableJITAsyncLoadingInitialBatchingFactor * AsyncLoadRequests.Num();
	int32 InitialBatchSize = FMath::Max(FMath::CeilToInt(ToBatch), 1);

	// Dequeue requests from queue before kicking off async loading so as to prevent any contention with the ALT.
	TArray<FStreamableJITAsyncLoadRequest> InitialBatchRequests;
	InitialBatchRequests.Reserve(InitialBatchSize);
	while(InitialBatchSize > 0)
	{
		FStreamableJITAsyncLoadRequest AsyncLoadRequest;
		if (!AsyncLoadRequests.Dequeue(AsyncLoadRequest))
		{
			break;
		}
		InitialBatchRequests.Emplace(MoveTemp(AsyncLoadRequest));
		--InitialBatchSize;
	}

	for (const FStreamableJITAsyncLoadRequest& Request : InitialBatchRequests)
	{
		KickLoadingRequest(Request);
	}
}

void FStreamableJustInTimeAsyncLoader::MakeProgress(const FLoadPackageAsyncProgressParams& InParams)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FStreamableJustInTimeAsyncLoader::MakeProgress);

	switch (InParams.ProgressType)
	{
	case EAsyncLoadingProgress::Serialized:
	case EAsyncLoadingProgress::Failed:
	{
		// This case is called when processing async load requests on the async load thread.
		check(IsInAsyncLoadingThread());
		if (!bIsCancelled.load(std::memory_order_relaxed))
		{
			// Queue subsequent requests as needed on the async load thread.
			if (!KickNextLoadingRequest())
			{
				Cleanup();
			}
		}
		else
		{
			Cleanup();
		}
		break;
	}
	case EAsyncLoadingProgress::FullyLoaded:
	{
		// This case is called when completing async load requests on the game thread.
		check(IsInGameThread());

		// Update priority and check for cancellation (safe to do so only on the game thread).
		UpdateHandleState();
		break;
	}
	default:
	{
		checkf(0, TEXT("Unexpected async loading progress type '%d'"), EnumToUnderlyingType(InParams.ProgressType));
		break;
	}		
	}
}

bool FStreamableJustInTimeAsyncLoader::KickNextLoadingRequest()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FStreamableJustInTimeAsyncLoader::KickNextLoadingRequest);

	FStreamableJITAsyncLoadRequest AsyncLoadRequest;
	if(AsyncLoadRequests.Dequeue(AsyncLoadRequest))
	{
		KickLoadingRequest(AsyncLoadRequest);
		return true;
	}

	return false;
}

void FStreamableJustInTimeAsyncLoader::KickLoadingRequest(const FStreamableJITAsyncLoadRequest& AsyncLoadRequest)
{
	const int32 LoadPriority = Priority.load(std::memory_order_relaxed);
	FLoadPackageAsyncOptionalParams LoadPackageAsyncOptionalParams =
	{
		.CompletionDelegate = MakeUnique<FLoadPackageAsyncDelegate>(AsyncLoadRequest.CompletionDelegate),
		.ProgressDelegate = ProgressDelegate,
		.PackagePriority = LoadPriority // Queue using updated priorities.
	};
	int32 RequestID = LoadPackageAsync(AsyncLoadRequest.PackagePath, MoveTemp(LoadPackageAsyncOptionalParams));
	AsyncLoadRequest.RequestQueuedDelegate.ExecuteIfBound(RequestID);
	UE_LOGF(LogStreamableAsyncLoading, Verbose, "FStreamableJustInTimeAsyncLoader queued async load request [Package:%ls] [Priority: %d] [RequestID]:%d", *AsyncLoadRequest.PackagePath.GetDebugName(), LoadPriority, RequestID);
	TotalRemainingRequests.fetch_sub(1, std::memory_order_relaxed);
}

void FStreamableJustInTimeAsyncLoader::UpdateHandleState()
{
	// Expected to be called on the game thread (see case EAsyncLoadingProgress::FullyLoaded).
	check(IsInGameThread());

	if (TSharedPtr<FStreamableHandle> Handle = OwningHandle.Pin())
	{
		if (Handle->IsUsingJustInTimeAsyncLoader())
		{
			// If still using this loader, update priorities of subsequent requests.
			Priority = Handle->GetPriority();
		}
		else
		{
			Cancel();
		}
	}
	else
	{
		Cancel();
	}
}

void FStreamableJustInTimeAsyncLoader::Cancel()
{
	// Expected to be called on the game thread (see case EAsyncLoadingProgress::FullyLoaded and UpdateHandleState).
	check(IsInGameThread());

	UE_CLOGF(OwningHandle.IsValid(), LogStreamableAsyncLoading, Verbose, 
		"FStreamableJustInTimeAsyncLoader [%ls] flagging for cancel [Remaining:%d]", 
		*OwningHandle.Pin()->GetDebugName(), TotalRemainingRequests.load(std::memory_order_relaxed));

#if UE_STREAMABLE_MANAGER_PROFILING_ENABLED
	FStreamableManagerProfiling::OnStreamablesCancelled(TotalRemainingRequests.load(std::memory_order_relaxed));
#endif

	bIsCancelled.store(true, std::memory_order_relaxed);
}

void FStreamableJustInTimeAsyncLoader::Cleanup()
{
	// Expected to be called on the async load thread (see case EAsyncLoadingProgress::Serialized or EAsyncLoadingProgress::Failed).
	check(IsInAsyncLoadingThread());
	
	UE_CLOGF(!AsyncLoadRequests.IsEmpty(), LogStreamableAsyncLoading, Verbose, "FStreamableJustInTimeAsyncLoader cancelled %d requests", TotalRemainingRequests.load(std::memory_order_relaxed));
	
	ProgressDelegate = nullptr;
}
