// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Async/Mutex.h"
#include "Misc/PackagePath.h"
#include "Containers/SpscQueue.h"
#include "UObject/UObjectGlobals.h"

using FStreamableAsyncLoaderRequestQueuedDelegate = TTSDelegate<void(int32)>;

struct FStreamableHandle;

/**
 * Interface that controls async loading behavior.
 */
class IStreamableAsyncLoader : public TSharedFromThis<IStreamableAsyncLoader>
{
public:
public:
	virtual ~IStreamableAsyncLoader() = default;

	/**
	* Adds a package for load to the existing queue. The package request may be started or queued until the call to `StartAsyncLoadRequests`.
	* @param	InPackagePath				The package to be loaded.
	* @param	InRequestQueuedDelegate		Called when each request is queued. Called on either the game thread or the async loading thread with the request ID for the pushed request.
	* @param	InCompletionDelegate		Completion callback to be called per when the package is fully loaded. Called on the game thread.
	* @param	InPriority					Priority of the request
	*/
	virtual void AddAsyncLoadRequest(const FPackagePath& InPackagePath, FStreamableAsyncLoaderRequestQueuedDelegate&& InRequestQueuedDelegate, FLoadPackageAsyncDelegate&& InCompletionDelegate, TAsyncLoadPriority InPriority) = 0;
	
	/**
	* Starts async load requests if not started earlier.
	*/
	virtual void StartAsyncLoadRequests() = 0;
};

/**
 * Default loader that starts async load requests when added.
 */
class FStreamableAsyncLoader final : public IStreamableAsyncLoader
{
public:
	// Begin IStreamableAsyncLoader interface.
	void AddAsyncLoadRequest(const FPackagePath& InPackagePath, FStreamableAsyncLoaderRequestQueuedDelegate&& InRequestQueuedDelegate, FLoadPackageAsyncDelegate&& InCompletionDelegate, TAsyncLoadPriority InPriority) override;
	void StartAsyncLoadRequests() override;
	// End IStreamableAsyncLoader interface.
};

/**
 * Loader kicks a subset of requests in `StartAsyncLoadRequests` while chaining subsequent load requests from the loading thread.
 * @see s.StreamableJITAsyncLoadingInitialBatchingFactor
 */
class FStreamableJustInTimeAsyncLoader final : public IStreamableAsyncLoader
{
public:
	FStreamableJustInTimeAsyncLoader(TWeakPtr<FStreamableHandle> Handle);
	~FStreamableJustInTimeAsyncLoader() override;

	// Begin IStreamableAsyncLoader interface.
	void AddAsyncLoadRequest(const FPackagePath& InPackagePath, FStreamableAsyncLoaderRequestQueuedDelegate&& InRequestQueuedDelegate, FLoadPackageAsyncDelegate&& InCompletionDelegate, TAsyncLoadPriority InPriority) override;
	void StartAsyncLoadRequests() override;
	// End IStreamableAsyncLoader interface.

private:
	struct FStreamableJITAsyncLoadRequest
	{
		FPackagePath PackagePath;
		FStreamableAsyncLoaderRequestQueuedDelegate RequestQueuedDelegate;
		FLoadPackageAsyncDelegate CompletionDelegate;
	};
	using FStreamableJITAsyncLoadRequests = TSpscQueue<FStreamableJITAsyncLoadRequest>;

	// Can be called on game thread or async loading thread.
	void MakeProgress(const FLoadPackageAsyncProgressParams& InParams);

	// Can be called on game thread (for initial batch) followed by subsequent calls being made on the async loading thread.
	bool KickNextLoadingRequest();
	void KickLoadingRequest(const FStreamableJITAsyncLoadRequest& AsyncLoadRequest);

	void UpdateHandleState();
	
	void Cancel();
	void Cleanup();
		
	FStreamableJITAsyncLoadRequests AsyncLoadRequests;
	TSharedPtr<FLoadPackageAsyncProgressDelegate> ProgressDelegate;
	std::atomic<TAsyncLoadPriority> Priority = 0;
	std::atomic<int32> TotalRemainingRequests = 0;

	// Do not pin on async load thread.
	// We cannot take ownership of the object on the async loading thread as FStreamableHandles are not thread safe.
	TWeakPtr<FStreamableHandle> OwningHandle;

	std::atomic<bool> bIsCancelled = false;
};
