// Copyright Epic Games, Inc. All Rights Reserved.

#include "Engine/StreamableManager.h"
#include "Experimental/UnifiedError/CoreErrorTypes.h"
#include "Experimental/UnifiedError/UnifiedError.h"
#include "Engine/Experimental/StreamableManagerError.h"
#include "Experimental/IO/IoStatusError.h"
#include "UObject/ICookInfo.h"
#include "UObject/ObjectRedirector.h"
#include "UObject/Package.h"
#include "Algo/Transform.h"
#include "Misc/PackageAccessTrackingOps.h"
#include "Misc/PackageName.h"
#include "Misc/ScopeLock.h"
#include "Misc/TransactionallySafeCriticalSection.h"
#include "UObject/UObjectThreadContext.h"
#include "HAL/IConsoleManager.h"
#include "Tickable.h"
#include "Serialization/LoadTimeTrace.h"
#include "ProfilingDebugging/CpuProfilerTrace.h"
#include "ProfilingDebugging/StreamableManagerProfiling.h"
#include "StreamableAsyncLoader.h"
#include "Trace/Trace.h"
#include "Trace/Trace.inl"
#include <type_traits>

DEFINE_LOG_CATEGORY_STATIC(LogStreamableManager, Log, All);


// Default to 1 frame, this will cause the delegates to go off on the next tick to avoid recursion issues. Set higher to fake disk lag
static int32 GStreamableDelegateDelayFrames = 1;
static FAutoConsoleVariableRef CVarStreamableDelegateDelayFrames(
	TEXT("s.StreamableDelegateDelayFrames"),
	GStreamableDelegateDelayFrames,
	TEXT("Number of frames to delay StreamableManager delegates "),
	ECVF_Default
);

// CVar to switch back to legacy behavior of non-specifically flushing async loading when waiting on a request handle
static bool GStreamableFlushAllAsyncLoadRequestsOnWait = 0;
static FAutoConsoleVariableRef CVarStreamableFlushAllAsyncLoadRequestsOnWait(
	TEXT("s.StreamableFlushAllAsyncLoadRequestsOnWait"),
	GStreamableFlushAllAsyncLoadRequestsOnWait,
	TEXT("Flush async loading without a specific request ID when waiting on streamable handles."),
	ECVF_Default
);

static bool GStreamableStripDebugName = false;
static FAutoConsoleVariableRef CVarStreamableStripDebugName(
	TEXT("s.StreamableStripDebugName"),
	GStreamableStripDebugName,
	TEXT("Whether to strip the bundle names DebugName when changing bundle. Saves memory when disabled, provides more debug information when enabled."),
	ECVF_Default
);

static bool GStreamableStripDebugNameInShipping = false;
static FAutoConsoleVariableRef CVarStreamableStripDebugNameInShipping(
	TEXT("s.StreamableStripDebugNameInShipping"),
	GStreamableStripDebugNameInShipping,
	TEXT("Whether to strip the bundle names DebugName when changing bundle. Saves memory when disabled, provides more debug information when enabled. Only applied to shipping builds."),
	ECVF_Default
);

static bool GStreamableLogAsyncLoadErrorsAsErrors = false;
static FAutoConsoleVariableRef CVarLogAsyncLoadErrorsAsErrors(
	TEXT("s.LogAsyncLoadErrorsAsErrors"),
	GStreamableLogAsyncLoadErrorsAsErrors,
	TEXT("Wether async loading errors should be logged as errors. If disabled, async load errors will be logged as verbose."),
	ECVF_Default
);

static bool GStreamableLogStreamableFailuresAsErrors = false;
static FAutoConsoleVariableRef CVarLogStreamableFailuresAsErrors(
		TEXT("s.LogStreamableFailuresAsErrors"),
	GStreamableLogStreamableFailuresAsErrors,
		TEXT("Log all streamable failures as errors."),
		ECVF_Default
	);

static bool GStreamableEnableJITAsyncLoading = false;
static FAutoConsoleVariableRef CVarStreamableEnableJITAsyncLoading(
	TEXT("s.StreamableEnableJITAsyncLoading"),
	GStreamableEnableJITAsyncLoading,
	TEXT("If enabled, streamable manager will trickle async load requests followed by just in time processing of subsequent requests, providing opportunities for reprioritization/cancels between requests"),
	ECVF_Default
);

static bool GStreamableEnableJITAsyncLoadingGlobally = false;
static FAutoConsoleVariableRef CVarStreamableEnableJITAsyncLoadingGlobally(
	TEXT("s.StreamableEnableJITAsyncLoadingGlobally"),
	GStreamableEnableJITAsyncLoadingGlobally,
	TEXT("If enabled, streamable manager will trickle async load requests regardless of if it was specified in the request parameters"),
	ECVF_Default
);

#if !UE_BUILD_SHIPPING
static FString GStreamableErrorOnRequest;
static FAutoConsoleVariableRef CVarStreamableErrorOnRequest(
		TEXT("s.errorOnRequest"),
		GStreamableErrorOnRequest,
		TEXT("If this is set and the errorOnRequest string partially matches the current debug name of the asset being loaded, then an error will be thrown instead of completion."),
		ECVF_Default
	);

static FString GStreamableDebugOnRequest = "";
static FAutoConsoleVariableRef CVarStreamableDebugOnRequest(
		TEXT("s.debugOnRequest"),
		GStreamableDebugOnRequest,
		TEXT("If this is set then a log will fire when any requested asset string matches in part this variable, set breakpoint to debug the caller."),
		ECVF_Default
	);
#endif
FStreamableDownloadCachePin::FStreamableDownloadCachePin() = default;
FStreamableDownloadCachePin::~FStreamableDownloadCachePin() = default;
FString FStreamableDownloadCachePin::GetDebugName() const { return LexToString(ContentHandle); }
void FStreamableDownloadCachePin::Release() { ContentHandle.Reset(); }
bool FStreamableDownloadCachePin::IsValid() const { return ContentHandle.IsValid(); }
bool FStreamableDownloadCachePin::operator==(FStreamableDownloadCachePin& Other) const
{
	return ContentHandle == Other.ContentHandle;
}

const FString FStreamableHandle::HandleDebugName_Preloading = FString("Preloading");
const FString FStreamableHandle::HandleDebugName_AssetList = FString("LoadAssetList");
const FString FStreamableHandle::HandleDebugName_CombinedHandle = FString("CreateCombinedHandle");
const FString FStreamableHandle::HandleDebugName_Error = FString("CreateErrorHandle");

/** Helper class that defers streamable manager delegates until the next frame */
class FStreamableDelegateDelayHelper : public FTickableGameObject
{
public:

	/** Adds a delegate to deferred list */
	template<typename InStreamableDelegate>
	void AddDelegate(InStreamableDelegate&& Delegate, InStreamableDelegate&& CancelDelegate, TSharedPtr<FStreamableHandle> AssociatedHandle)
	{
		FPendingDelegate* PendingDelegate = new FPendingDelegate(Forward<InStreamableDelegate>(Delegate), Forward<InStreamableDelegate>(CancelDelegate), AssociatedHandle);

		{
			UE::TScopeLock Lock(DataLock);

			FPendingDelegateList& DelegatesForHandle = PendingDelegatesByHandle.FindOrAdd(AssociatedHandle);
			FPendingDelegateList::AddTail(PendingDelegate, PendingDelegates, DelegatesForHandle);
		}
	}

	/** Cancels delegate for handle, this will either delete the delegate or replace with the cancel delegate */
	void CancelDelegatesForHandle(TSharedPtr<FStreamableHandle> AssociatedHandle)
	{
		FPendingDelegateList PendingDelegatesToDelete;

		{
			UE::TScopeLock _(DataLock);

			FPendingDelegateList* DelegatesForHandle = PendingDelegatesByHandle.Find(AssociatedHandle);
			if (!DelegatesForHandle)
			{
				return;
			}

			FPendingDelegate* CurrentNode = DelegatesForHandle->Head;
			while (CurrentNode)
			{
				FPendingDelegate* NextNode = CurrentNode->NextByHandle;
				if (CurrentNode->CancelDelegate.IsBound())
				{
					// Replace with cancel delegate
					CurrentNode->Delegate = CurrentNode->CancelDelegate;
					CurrentNode->CancelDelegate.Unbind();
				}
				else
				{
					// Remove entirely
					FPendingDelegateList::Unlink(CurrentNode, PendingDelegates, *DelegatesForHandle);
					if (PendingDelegatesToDelete.Tail)
					{
						PendingDelegatesToDelete.Tail->Next = CurrentNode;
						PendingDelegatesToDelete.Tail = CurrentNode;
					}
					else
					{
						PendingDelegatesToDelete.Head = PendingDelegatesToDelete.Tail = CurrentNode;
					}
					CurrentNode->Next = nullptr;
				}
				CurrentNode = NextNode;
			}
			if (!DelegatesForHandle->Head)
			{
				RemovePendingDelegateInternal(AssociatedHandle);
			}
		}

		FPendingDelegate* NodeToDelete = PendingDelegatesToDelete.Head;
		while (NodeToDelete)
		{
			FPendingDelegate* NextNodeToDelete = NodeToDelete->Next;
			delete NodeToDelete;
			NodeToDelete = NextNodeToDelete;
		}
	}

	/** Calls all delegates, call from synchronous flushes */
	void FlushDelegates()
	{
		if (!PendingDelegates.Head)
		{
			return;
		}

		if (AutoRTFM::IsTransactional())
		{
			ensureMsgf(false, TEXT("%hs was not expected to be called inside a transaction (it won't complete until the tx ends, so no side effects will be visible during the tx)"), __func__);

			AutoRTFM::OnCommit([this]()
			{
				FlushDelegates();
			});

			return;
		}

		while (PendingDelegates.Head)
		{
			Tick(0.0f);
		}
	}

	// FTickableGameObject interface

	void Tick(float DeltaTime) override
	{
		if (!PendingDelegates.Head)
		{
			return;
		}

		if (AutoRTFM::IsTransactional())
		{
			// Complete this transaction, then call this function again outside the transaction
			AutoRTFM::OnCommit([this, DeltaTime]()
			{
				Tick(DeltaTime);
			});

			return;
		}

		FPendingDelegateList DelegatesToCall;
		{
			UE::TScopeLock Lock(DataLock);

			FPendingDelegate* CurrentNode = PendingDelegates.Head;
			while (CurrentNode)
			{
				FPendingDelegate* NextNode = CurrentNode->Next;
				if (--CurrentNode->DelayFrames <= 0)
				{
					// Add to call array and remove from tracking one
					FPendingDelegateList* DelegatesForHandle = PendingDelegatesByHandle.Find(CurrentNode->RelatedHandle);
					check(DelegatesForHandle);
					FPendingDelegateList::Unlink(CurrentNode, PendingDelegates, *DelegatesForHandle);
					if (DelegatesToCall.Tail)
					{
						DelegatesToCall.Tail->Next = CurrentNode;
						DelegatesToCall.Tail = CurrentNode;
					}
					else
					{
						DelegatesToCall.Head = DelegatesToCall.Tail = CurrentNode;
					}
					if (!DelegatesForHandle->Head)
					{
						RemovePendingDelegateInternal(CurrentNode->RelatedHandle);
					}
				}
				CurrentNode = NextNode;
			}
		}

		FPendingDelegate* DelegateToCall = DelegatesToCall.Head;
		while (DelegateToCall)
		{
			FPendingDelegate* NextDelegateToCall = DelegateToCall->Next;
			DelegateToCall->Delegate.ExecuteIfBound(DelegateToCall->RelatedHandle);
			delete DelegateToCall;
			DelegateToCall = NextDelegateToCall;
		}
	}

	virtual ETickableTickType GetTickableTickType() const override
	{
		return ETickableTickType::Always;
	}

	virtual bool IsTickableWhenPaused() const override
	{
		return true;
	}

	virtual bool IsTickableInEditor() const
	{
		return true;
	}

	virtual TStatId GetStatId() const override
	{
		RETURN_QUICK_DECLARE_CYCLE_STAT(FStreamableDelegateDelayHelper, STATGROUP_Tickables);
	}

private:

	struct FPendingDelegate
	{
		FPendingDelegate* Prev;
		FPendingDelegate* Next;

		FPendingDelegate* PrevByHandle;
		FPendingDelegate* NextByHandle;

		/** Delegate to call when frames are up */
		FStreamableDelegateWithHandle Delegate;

		/** Delegate to call if this gets cancelled early */
		FStreamableDelegateWithHandle CancelDelegate;

		/** Handle related to delegates, needs to keep these around to avoid things GCing before the user callback goes off. This may be null */
		TSharedPtr<FStreamableHandle> RelatedHandle;

		/** Frames left to delay */
		int32 DelayFrames;

		template<typename InStreamableDelegate>
		FPendingDelegate(InStreamableDelegate&& InDelegate, InStreamableDelegate&& InCancelDelegate, TSharedPtr<FStreamableHandle> InHandle)
			: Delegate(Forward<InStreamableDelegate>(InDelegate))
			, CancelDelegate(Forward<InStreamableDelegate>(InCancelDelegate))
			, RelatedHandle(MoveTemp(InHandle))
			, DelayFrames(GStreamableDelegateDelayFrames)
		{}
	};

	struct FPendingDelegateList
	{
		FPendingDelegate* Head = nullptr;
		FPendingDelegate* Tail = nullptr;

		static void Unlink(FPendingDelegate* PendingDelegate, FPendingDelegateList& PendingDelegates, FPendingDelegateList& PendingDelegatesByHandle)
		{
			if (PendingDelegate->Prev)
			{
				PendingDelegate->Prev->Next = PendingDelegate->Next;
			}
			else
			{
				check(PendingDelegate == PendingDelegates.Head);
				PendingDelegates.Head = PendingDelegate->Next;
			}
			if (PendingDelegate->Next)
			{
				PendingDelegate->Next->Prev = PendingDelegate->Prev;
			}
			else
			{
				check(PendingDelegate == PendingDelegates.Tail);
				PendingDelegates.Tail = PendingDelegate->Prev;
			}
			PendingDelegate->Next = PendingDelegate->Prev = nullptr;

			if (PendingDelegate->PrevByHandle)
			{
				PendingDelegate->PrevByHandle->NextByHandle = PendingDelegate->NextByHandle;
			}
			else
			{
				check(PendingDelegate == PendingDelegatesByHandle.Head);
				PendingDelegatesByHandle.Head = PendingDelegate->NextByHandle;
			}
			if (PendingDelegate->NextByHandle)
			{
				PendingDelegate->NextByHandle->PrevByHandle = PendingDelegate->PrevByHandle;
			}
			else
			{
				check(PendingDelegate == PendingDelegatesByHandle.Tail);
				PendingDelegatesByHandle.Tail = PendingDelegate->PrevByHandle;
			}
			PendingDelegate->NextByHandle = PendingDelegate->PrevByHandle = nullptr;

		}

		static void AddTail(FPendingDelegate* PendingDelegate, FPendingDelegateList& PendingDelegates, FPendingDelegateList& PendingDelegatesByHandle)
		{
			if (PendingDelegates.Tail)
			{
				PendingDelegates.Tail->Next = PendingDelegate;
				PendingDelegate->Prev = PendingDelegates.Tail;
				PendingDelegate->Next = nullptr;
				PendingDelegates.Tail = PendingDelegate;
			}
			else
			{
				check(!PendingDelegates.Head);
				PendingDelegates.Head = PendingDelegates.Tail = PendingDelegate;
				PendingDelegate->Next = PendingDelegate->Prev = nullptr;
			}

			if (PendingDelegatesByHandle.Tail)
			{
				PendingDelegatesByHandle.Tail->NextByHandle = PendingDelegate;
				PendingDelegate->PrevByHandle = PendingDelegatesByHandle.Tail;
				PendingDelegate->NextByHandle = nullptr;
				PendingDelegatesByHandle.Tail = PendingDelegate;
			}
			else
			{
				check(!PendingDelegatesByHandle.Head);
				PendingDelegatesByHandle.Head = PendingDelegatesByHandle.Tail = PendingDelegate;
				PendingDelegate->NextByHandle = PendingDelegate->PrevByHandle = nullptr;
			}
		}
	};

	inline void RemovePendingDelegateInternal(TSharedPtr<FStreamableHandle> Handle)
	{
		// requires DataLock by caller
		PendingDelegatesByHandle.Remove(Handle);
		if (PendingDelegatesByHandle.IsEmpty())
		{
			// release potentially big allocation after huge batches of streaming requests
			PendingDelegatesByHandle.Empty(128);
		}
	}

	FPendingDelegateList PendingDelegates;
	TMap<TSharedPtr<FStreamableHandle>, FPendingDelegateList> PendingDelegatesByHandle;

	FTransactionallySafeCriticalSection DataLock;
};

static FStreamableDelegateDelayHelper* StreamableDelegateDelayHelper = nullptr;


TStreamableHandleContextDataTypeID FStreamableHandleContextDataBase::AllocateClassTypeId()
{
	static std::atomic<TStreamableHandleContextDataTypeID> TypeNum{0};
	TStreamableHandleContextDataTypeID Result = TypeNum++;

	checkf(Result != TStreamableHandleContextDataTypeIDInvalid, TEXT("Overflow in TypeNum: too many TStreamableHandleContextData subclasses. Change the TStreamableHandleContextDataTypeID typedef if more subclasses are needed."));
	return Result;
}

namespace UE::StreamableManager::Private
{
	// self-deleting context for managing package downloads for a load request
	struct FDownloadContext
	{
		UE::IoStore::FOnDemandInstallRequest InstallRequest;
		UE::IoStore::FOnDemandInstallProgress InstallProgress;
		FStreamableDownloadPriority Priority = FStreamableManager::DownloadDefaultPriority;
		bool bInstallSoftReferences = false;
		bool bDownloadOnly = false;
		bool bAllowMissingDependencies = false;

		bool IsStarted() const
		{
			return !InstallRequest.IsNone();
		}

		~FDownloadContext()
		{
			InstallRequest.Cancel();
		}

		void Cancel(const TSharedRef<FStreamableHandle>& OwningHandle)
		{
			if (IsStarted())
			{
				// Will be cleaned up when request calls back
				InstallRequest.Cancel();
			}
			else
			{
				// No callback coming, destroy self now
				OwningHandle->DownloadContext = nullptr;
			}
		}

		void CompleteLoad(const TSharedRef<FStreamableHandle>& OwningHandle)
		{
			OwningHandle->CompleteLoad();
			if (OwningHandle->bReleaseWhenLoaded)
			{
				OwningHandle->ReleaseHandle();
			}
		}

		void CompleteLoad(const TSharedRef<FStreamableHandle>& OwningHandle, UE::UnifiedError::FError&& Error)
		{
			OwningHandle->SetError(MoveTemp(Error));
			CompleteLoad(OwningHandle);
		}

		void Start(const TSharedRef<FStreamableHandle>& OwningHandle)
		{
			if (!InstallRequest.IsNone())
			{
				checkf(false, TEXT("IAD install already started"));
				return;
			}

			UE::IoStore::IOnDemandIoStore* IoStore = UE::IoStore::TryGetOnDemandIoStore();
			if (!IoStore)
			{
				CompleteLoad(OwningHandle, ::StreamableManager::IoStoreNotFound());
				OwningHandle->DownloadContext = nullptr; // Destroy self
				return;
			}

			UE::IoStore::FOnDemandInstallArgs InstallArgs;
			InstallArgs.PackageIds.Reserve(OwningHandle->RequestedAssets.Num());
			Algo::Transform(OwningHandle->RequestedAssets, InstallArgs.PackageIds, [](const FSoftObjectPath& AssetPath)
				{
					return FPackageId::FromName(AssetPath.GetLongPackageFName());
				});

			check(OwningHandle->DownloadCachePin);
			InstallArgs.ContentHandle = OwningHandle->DownloadCachePin.ContentHandle;

			InstallArgs.Options |= UE::IoStore::EOnDemandInstallOptions::CallbackOnGameThread;
			if (bInstallSoftReferences)
			{
				InstallArgs.Options |= UE::IoStore::EOnDemandInstallOptions::InstallSoftReferences;
			}
			if (bAllowMissingDependencies)
			{
				InstallArgs.Options |= UE::IoStore::EOnDemandInstallOptions::AllowMissingDependencies;
			}

			InstallArgs.Priority = Priority;

			
			InstallRequest = IoStore->Install(MoveTemp(InstallArgs),
				[WeakOwningHandle = OwningHandle.ToWeakPtr(), DebugString = OwningHandle->GetDebugName()](UE::IoStore::FOnDemandInstallResult&& OnDemandInstallResult)
				{
					TSharedPtr<FStreamableHandle> OwningHandle = WeakOwningHandle.Pin();
					if (OwningHandle && OwningHandle->DownloadContext)
					{
						OwningHandle->DownloadContext->OnInstallComplete(OwningHandle.ToSharedRef(), MoveTemp(OnDemandInstallResult));
					}
					else
					{
						UE_LOGF(LogStreamableManager, Warning, "Owning handle was cleaned up before completion event was run. Owning handle name %ls", *DebugString);
					}
				},
				[WeakOwningHandle = OwningHandle.ToWeakPtr()](const UE::IoStore::FOnDemandInstallProgress& InProgress)
				{
					TSharedPtr<FStreamableHandle> OwningHandle = WeakOwningHandle.Pin();
					if (OwningHandle && OwningHandle->DownloadContext)
					{
						OwningHandle->DownloadContext->OnInstallProgress(OwningHandle.ToSharedRef(), InProgress);
					}
				}
			);
		}

		void OnInstallComplete(const TSharedRef<FStreamableHandle>& OwningHandle, UE::IoStore::FOnDemandInstallResult&& OnDemandInstallResult)
		{
			// Destroy self at end of scope
			TUniquePtr<FDownloadContext> Self = MoveTemp(OwningHandle->DownloadContext);
			OwningHandle->DownloadContext = nullptr;

			if (!OnDemandInstallResult.IsOk())
			{
				if (OwningHandle->WasCanceled() && UE::Core::IsCancellationError(OnDemandInstallResult.Error.GetValue()))
				{
					CompleteLoad(OwningHandle);
				}
				else
				{
					CompleteLoad(OwningHandle, MoveTemp(OnDemandInstallResult.Error.GetValue()));
				}

				return;
			}

			if (bDownloadOnly)
			{
				CompleteLoad(OwningHandle);
				return;
			}

			if (OwningHandle->OwningManager)
			{
				// StartHandleRequests will progress to asset loading now that the download context is gone
				OwningHandle->OwningManager->StartHandleRequests(OwningHandle->AsShared());
			}
		}

		void OnInstallProgress(const TSharedRef<FStreamableHandle>& OwningHandle, const UE::IoStore::FOnDemandInstallProgress& InProgress)
		{
			InstallProgress = InProgress;

			if (OwningHandle->OwningManager) // Make sure handle isn't canceled or released
			{
				OwningHandle->CallUpdateDelegate();
			}
		}
	};
}

FStreamableHandle::FStreamableHandle(FPrivateToken, UE::FSourceLocation&& InLocation)
	: Location(MoveTemp(InLocation))
	, Priority(0)
	, StreamablesLoading(0)
	, OwningManager(nullptr)
#if WITH_EDITOR
	, CookLoadType(ECookLoadType::Unspecified)
#endif
	, bLoadStarted(false)
	, bLoadCompleted(false)
	, bReleased(false)
	, bCanceled(false)
	, bStalled(false)
	, bReleaseWhenLoaded(false)
	, bIsCombinedHandle(false)
{
}

bool FStreamableHandle::BindCompleteDelegate(FStreamableDelegateWithHandle NewDelegate)
{
	if (!IsLoadingInProgress())
	{
		// Too late!
		return false;
	}

	CompleteDelegate = MoveTemp(NewDelegate);
	return true;
}

bool FStreamableHandle::BindCompleteDelegate(FStreamableDelegate NewDelegate)
{
	if (!IsLoadingInProgress())
	{
		// Too late!
		return false;
	}

	return BindCompleteDelegate(
		UE::StreamableManager::Private::WrapDelegate(MoveTemp(NewDelegate)));
}

bool FStreamableHandle::BindCancelDelegate(FStreamableDelegateWithHandle NewDelegate)
{
	if (!IsLoadingInProgress())
	{
		// Too late!
		return false;
	}

	CancelDelegate = MoveTemp(NewDelegate);
	return true;
}

bool FStreamableHandle::BindCancelDelegate(FStreamableDelegate NewDelegate)
{
	if (!IsLoadingInProgress())
	{
		// Too late!
		return false;
	}

	return BindCancelDelegate(
		UE::StreamableManager::Private::WrapDelegate(MoveTemp(NewDelegate)));
}

bool FStreamableHandle::BindUpdateDelegate(FStreamableUpdateDelegate NewDelegate)
{
	if (!IsLoadingInProgress())
	{
		// Too late!
		return false;
	}

	UpdateDelegate = MoveTemp(NewDelegate);
	return true;
}

EAsyncPackageState::Type FStreamableHandle::WaitUntilComplete(float Timeout, bool bStartStalledHandles)
{
	if (HasLoadCompleted())
	{
		return EAsyncPackageState::Complete;
	}

	// We need to recursively start any stalled handles
	if (bStartStalledHandles)
	{
		TArray<TSharedRef<FStreamableHandle>> HandlesToStart;

		HandlesToStart.Add(AsShared());

		for (int32 i = 0; i < HandlesToStart.Num(); i++)
		{
			TSharedRef<FStreamableHandle> Handle = HandlesToStart[i];

			if (Handle->IsStalled())
			{
				// If we were stalled, start us now to avoid deadlocks
				UE_LOGF(LogStreamableManager, Warning, "FStreamableHandle::WaitUntilComplete called on stalled handle %ls, forcing load even though resources may not have been acquired yet", *Handle->GetDebugName());
				Handle->StartStalledHandle();
			}

			for (const TSharedPtr<FStreamableHandle>& ChildHandle : Handle->ChildHandles)
			{
				if (ChildHandle.IsValid())
				{
					HandlesToStart.Add(ChildHandle.ToSharedRef());
				}
			}
		}
	}

	// If we have have a timeout we can't call FlushAsyncLoading so use ProcessAsyncLoadingUntilComplete 
	if (Timeout == 0.0f && !GStreamableFlushAllAsyncLoadRequestsOnWait)
	{
#if UE_ENABLE_STREAMABLE_JIT_ASYNC_LOADING
		// Below queues pending requests without using the JIT async loader so we can flush these below.
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(FStreamableManager::CancelJITLoadsRecursively);

			// Recursively disable JIT loads and enqueue pending async load requests if needed.
			OwningManager->CancelJITLoadsRecursively(AsShared());
		}
#endif // UE_ENABLE_STREAMABLE_JIT_ASYNC_LOADING
		TArray<int32> RequestIds = OwningManager->GetAsyncLoadRequestIds(AsShared());
		FlushAsyncLoading(RequestIds);	
		if (ensureMsgf(HasLoadCompletedOrStalled(), TEXT("Flushing async loading by request id did not complete loading for streamable handle %s"), *GetDebugName()))
		{
			return EAsyncPackageState::Complete;
		}
		// If for some reason the streamables don't consider themselves complete, fall back to old codepath which flushes asyncing loading without specific IDs
	}
	 
	// Finish when all handles are completed or stalled. If we started stalled above then there will be no stalled handles
	EAsyncPackageState::Type State = ProcessAsyncLoadingUntilComplete([this]() { return HasLoadCompletedOrStalled(); }, Timeout);
	
	if (State == EAsyncPackageState::Complete)
	{
		ensureMsgf(HasLoadCompletedOrStalled() || WasCanceled(), TEXT("WaitUntilComplete failed for streamable handle %s, async loading is done but handle is not complete"), *GetDebugName());
	}
	return State;
}

TArray<FStreamableDownloadCachePin> FStreamableHandle::GetDownloadCachePins() const
{
	TArray<FStreamableDownloadCachePin> Pins;
	
	TArray<const FStreamableHandle*, TInlineAllocator<32>> Stack;
	Stack.Push(this);

	do
	{
		const FStreamableHandle* Current = Stack.Pop(EAllowShrinking::No);
		if (Current->DownloadCachePin.IsValid())
		{
			Pins.AddUnique(Current->DownloadCachePin);
		}

		for (const TSharedPtr<FStreamableHandle>& ChildHandle : Current->ChildHandles)
		{
			Stack.Push(ChildHandle.Get());
		}
	} while (!Stack.IsEmpty());

	return Pins;
}

bool FStreamableHandle::HasLoadCompletedOrStalled() const
{
	// Check this handle
	if (!IsCombinedHandle() && !HasLoadCompleted() && !IsStalled())
	{
		return false;
	}

	// Check children recursively
	for (const TSharedPtr<FStreamableHandle>& ChildHandle : ChildHandles)
	{
		if (ChildHandle.IsValid() && !ChildHandle->HasLoadCompletedOrStalled())
		{
			return false;
		}
	}

	return true;
}

bool FStreamableHandle::IsHandleNameEmptyOrDefault() const
{
	// empty or "None"
	if (DebugName.IsEmpty() || DebugName.Equals(FName(NAME_None).ToString()))
	{
		return true;
	}

	// a default value...
	if (DebugName == HandleDebugName_AssetList || DebugName == HandleDebugName_CombinedHandle)
	{
		return true;
	}
	// a default generated debug name
	else if (DebugName.StartsWith(HandleDebugName_Preloading))
	{
		return true;
	}

	return false;
}

void FStreamableHandle::SetDebugNameIfEmptyOrDefault(const FString& NewName)
{
	if (IsHandleNameEmptyOrDefault())
	{
		DebugName = NewName;
	}
}

void FStreamableHandle::SetPriority(TAsyncLoadPriority InPriority)
{
	Priority = InPriority;
}

TOptional<FStreamableDownloadPriority> FStreamableHandle::GetDownloadPriority() const
{
	if (IsActive() && DownloadContext)
	{
		return DownloadContext->Priority;
	}

	return {};
}

bool FStreamableHandle::SetDownloadPriority(FStreamableDownloadPriority InPriority)
{
	if (IsActive() && DownloadContext)
	{
		DownloadContext->Priority = InPriority;
		DownloadContext->InstallRequest.UpdatePriority(DownloadContext->Priority);
		return true;
	}

	return false;
}

void FStreamableHandle::GetRequestedAssets(TArray<FSoftObjectPath>& AssetList, bool bIncludeChildren /*= true*/) const
{
	AssetList = RequestedAssets;

	// Check child handles

	if (bIncludeChildren)
	{
		for (const TSharedPtr<FStreamableHandle>& ChildHandle : ChildHandles)
		{
			TArray<FSoftObjectPath> ChildAssetList;

			ChildHandle->GetRequestedAssets(ChildAssetList);

			for (const FSoftObjectPath& ChildRef : ChildAssetList)
			{
				AssetList.AddUnique(ChildRef);
			}
		}
	}
}

UObject* FStreamableHandle::GetLoadedAsset() const
{
	TArray<UObject *> LoadedAssets;

	GetLoadedAssets(LoadedAssets);

	if (LoadedAssets.Num() > 0)
	{
		return LoadedAssets[0];
	}

	return nullptr;
}

void FStreamableHandle::GetLoadedAssets(TArray<UObject *>& LoadedAssets) const
{
	ForEachLoadedAsset([&LoadedAssets](UObject* LoadedAsset)
	{
		LoadedAssets.Add(LoadedAsset);
	});
}

void FStreamableHandle::GetLoadedCount(int32& LoadedCount, int32& RequestedCount) const
{
	RequestedCount = RequestedAssets.Num();
	LoadedCount = RequestedCount - StreamablesLoading;
	
	// Check child handles
	for (const TSharedPtr<FStreamableHandle>& ChildHandle : ChildHandles)
	{
		int32 ChildRequestedCount = 0;
		int32 ChildLoadedCount = 0;

		ChildHandle->GetLoadedCount(ChildLoadedCount, ChildRequestedCount);

		LoadedCount += ChildLoadedCount;
		RequestedCount += ChildRequestedCount;
	}
}

float FStreamableHandle::GetProgress(EStreamableProgressType Type) const
{
	if (Type == EStreamableProgressType::Load)
	{
		if (HasLoadCompleted())
		{
			return 1.0f;
		}

		int32 Loaded, Total;

		GetLoadedCount(Loaded, Total);

		return (float)Loaded / Total;
	}

	int32 DownloadCount = 0;
	float TotalProgress = 0.0f;

	TArray<const FStreamableHandle*, TInlineAllocator<32>> Stack;
	Stack.Push(this);

	do
	{
		const FStreamableHandle* Current = Stack.Pop(EAllowShrinking::No);
		if (Current->DownloadContext)
		{
			++DownloadCount;
			if (Type == EStreamableProgressType::DownloadRelative)
			{
				TotalProgress += Current->DownloadContext->InstallProgress.GetRelativeProgress();
			}
			else
			{
				TotalProgress += Current->DownloadContext->InstallProgress.GetAbsoluteProgress();
			}
		}

		for (const TSharedPtr<FStreamableHandle>& ChildHandle : Current->ChildHandles)
		{
			Stack.Push(ChildHandle.Get());
		}
	} while (!Stack.IsEmpty());

	if (DownloadCount > 0)
	{
		return TotalProgress / DownloadCount;
	}

	return 1.0f;
}

FStreamableManager* FStreamableHandle::GetOwningManager() const
{
	check(IsInGameThread());

	if (IsActive())
	{
		return OwningManager;
	}
	return nullptr;
}

void FStreamableHandle::CancelHandle()
{
	check(IsInGameThread());

	if (bCanceled || !OwningManager)
	{
		// Too late to cancel it
		return;
	}

	TSharedRef<FStreamableHandle> SharedThis = AsShared();
	if (bLoadCompleted || bReleased)
	{
		// Cancel if it's in the pending queue
		if (StreamableDelegateDelayHelper)
		{
			StreamableDelegateDelayHelper->CancelDelegatesForHandle(SharedThis);
		}

		if (!bReleased)
		{
			ReleaseHandle();
		}

		bCanceled = true;
		NotifyParentsOfCancellation();
		return;
	}

	bCanceled = true;
	NotifyParentsOfCancellation();

	ExecuteDelegate(MoveTemp(CancelDelegate), SharedThis);
	UnbindDelegates();

	// Remove from referenced list. If it is stalled then it won't have been registered with
	// the manager yet
	if (bLoadStarted)
	{
		for (const FSoftObjectPath& AssetRef : RequestedAssets)
		{
			OwningManager->RemoveReferencedAsset(AssetRef, SharedThis);
		}
	}

	bUseJustInTimeAsyncLoader = false;

	if (DownloadContext)
	{
		// This may destroy DownloadContext if its not pending
		DownloadContext->Cancel(AsShared());
	}

	// Remove from explicit list
	OwningManager->ManagedActiveHandles.Remove(SharedThis);

	// Remove child handles
	for (TSharedPtr<FStreamableHandle>& ChildHandle : ChildHandles)
	{
		ChildHandle->ParentHandles.Remove(SharedThis);
	}

	ChildHandles.Empty();
	CompletedChildCount = 0;
	CanceledChildCount = 0;

	OwningManager = nullptr;

	if (ParentHandles.Num() > 0)
	{
		// Update any meta handles that are still active. Copy the array first as elements may be removed from original while iterating
		TArray<TWeakPtr<FStreamableHandle>> ParentHandlesCopy = ParentHandles;
		for (TWeakPtr<FStreamableHandle>& WeakHandle : ParentHandlesCopy)
		{
			TSharedPtr<FStreamableHandle> Handle = WeakHandle.Pin();

			if (Handle.IsValid())
			{
				Handle->UpdateCombinedHandle();
			}
		}
	}
}

void FStreamableHandle::ReleaseHandle()
{
	check(IsInGameThread());

	if (bReleased || bCanceled)
	{
		// Too late to release it
		return;
	}

	check(OwningManager);

	if (bLoadCompleted)
	{
		bReleased = true;

		TSharedRef<FStreamableHandle> SharedThis = AsShared();

		if (bLoadStarted)
		{
			// Remove from referenced list
			for (const FSoftObjectPath& AssetRef : RequestedAssets)
			{
				OwningManager->RemoveReferencedAsset(AssetRef, SharedThis);
			}
		}

		// Remove from explicit list
		OwningManager->ManagedActiveHandles.Remove(SharedThis);

		// Remove child handles
		for (TSharedPtr<FStreamableHandle> ChildHandle : ChildHandles)
		{
			ChildHandle->ParentHandles.Remove(SharedThis);
		}

		ChildHandles.Empty();
		CompletedChildCount = 0;
		CanceledChildCount = 0;

		OwningManager = nullptr;
		bUseJustInTimeAsyncLoader = false;
	}
	else
	{
		// Set to release on complete
		bReleaseWhenLoaded = true;
		bUseJustInTimeAsyncLoader = false;
	}
}

void FStreamableHandle::StartStalledHandle()
{
	if (!bStalled || !IsActive())
	{
		// Cannot start
		return;
	}

	check(OwningManager);

	bStalled = false;
	OwningManager->StartHandleRequests(AsShared());
}

bool FStreamableHandle::HasCompleteDelegate() const
{
	return CompleteDelegate.IsBound();
}

bool FStreamableHandle::HasCancelDelegate() const
{
	return CancelDelegate.IsBound();
}

bool FStreamableHandle::HasUpdateDelegate() const
{
	return UpdateDelegate.IsBound();
}

static void RemoveActiveHandle(FStreamable& Streamable, FStreamableHandle& Handle);		

FStreamableHandle::~FStreamableHandle()
{
	checkf(IsInGameThread(), TEXT("Streamable handles aren't thread-safe and must be destroyed on the game thread"));

	if (IsActive() && bLoadStarted)
	{
		for (const FSoftObjectPath&  RequestedAsset : RequestedAssets)
		{
			if (FStreamable* Streamable = OwningManager->FindStreamable(RequestedAsset))
			{
				RemoveActiveHandle(*Streamable, *this);
			}
		}

		bReleased = true;
		OwningManager = nullptr;
		
		// The weak pointers in FStreamable will be nulled, but they're fixed on next GC, and actively canceling is not safe as we're halfway destroyed
	}
}

void FStreamableHandle::CompleteLoad()
{
	if (Error.IsSet())
	{
		if (GStreamableLogStreamableFailuresAsErrors)
		{
			UE_LOGF(LogStreamableManager, Error, "Encountered error: %ls", *Error->GetErrorMessage(true).ToString());
		}
		else
		{
			UE_LOGF(LogStreamableManager, Verbose, "Encountered error: %ls", *Error->GetErrorMessage(true).ToString());
		}
	}
	// Only complete if it's still active
	if (IsActive())
	{
		bLoadCompleted = true;

		bUseJustInTimeAsyncLoader = false;

		ExecuteDelegate(MoveTemp(CompleteDelegate), AsShared(), MoveTemp(CancelDelegate));
		UnbindDelegates();

		NotifyParentsOfCompletion();

		if (ParentHandles.Num() > 0)
		{
			// Update any meta handles that are still active. Copy the array first as elements may be removed from original while iterating
			TArray<TWeakPtr<FStreamableHandle>> ParentHandlesCopy = ParentHandles;
			for (TWeakPtr<FStreamableHandle>& WeakHandle : ParentHandlesCopy)
			{
				TSharedPtr<FStreamableHandle> Handle = WeakHandle.Pin();

				if (Handle.IsValid())
				{
					Handle->UpdateCombinedHandle();
				}
			}
		}
	}
}

void FStreamableHandle::NotifyParentsOfCompletion()
{
	// Update any meta handles that are still active.
	for (TWeakPtr<FStreamableHandle>& WeakHandle : ParentHandles)
	{
		TSharedPtr<FStreamableHandle> Handle = WeakHandle.Pin();

		if (Handle.IsValid())
		{
			++Handle->CompletedChildCount;
		}
	}
}

void FStreamableHandle::NotifyParentsOfCancellation()
{
	// Update any meta handles that are still active.
	for (TWeakPtr<FStreamableHandle>& WeakHandle : ParentHandles)
	{
		TSharedPtr<FStreamableHandle> Handle = WeakHandle.Pin();

		if (Handle.IsValid())
		{
			++Handle->CanceledChildCount;
		}
	}
}

void FStreamableHandle::UpdateCombinedHandle()
{
	if (!IsActive())
	{
		return;
	}

	if (!ensure(IsCombinedHandle()))
	{
		return;
	}

	if (CompletedChildCount + CanceledChildCount < ChildHandles.Num())
	{
		return;
	}

	// Check all our children, complete if done
	bool bAllCompleted = true;
	bool bAllCanceled = true;
	for (const TSharedPtr<FStreamableHandle>& ChildHandle : ChildHandles)
	{
		bAllCompleted = bAllCompleted && !ChildHandle->IsLoadingInProgress();
		bAllCanceled = bAllCanceled && ChildHandle->WasCanceled();

		if (!bAllCompleted && !bAllCanceled)
		{
			return;
		}
	}

	// If all our sub handles were canceled, cancel us. Otherwise complete us if at least one was completed and there are none in progress
	if (bAllCanceled)
	{
		if (OwningManager)
		{
			OwningManager->PendingCombinedHandles.Remove(AsShared());
		}

		CancelHandle();
	}
	else if (bAllCompleted)
	{
		if (OwningManager)
		{
			OwningManager->PendingCombinedHandles.Remove(AsShared());
		}

		CompleteLoad();

		if (bReleaseWhenLoaded)
		{
			ReleaseHandle();
		}
	}
}

void FStreamableHandle::CallUpdateDelegate()
{
	UpdateDelegate.ExecuteIfBound(AsShared());

	// Update any meta handles that are still active
	for (TWeakPtr<FStreamableHandle>& WeakHandle : ParentHandles)
	{
		TSharedPtr<FStreamableHandle> Handle = WeakHandle.Pin();

		if (Handle.IsValid())
		{
			Handle->CallUpdateDelegate();
		}
	}
}

void FStreamableHandle::UnbindDelegates()
{
	CancelDelegate.Unbind();
	UpdateDelegate.Unbind();
	CompleteDelegate.Unbind();
}

void FStreamableHandle::SetError(const UE::UnifiedError::FError& InError)
{
	if (!Error)
	{
		Error = InError;
		// add some additional context for this specific request 
		::StreamableManager::FRequestContext RequestContext = { DebugName, RequestedAssets, DownloadCachePin.IsValid() };
		Error->PushErrorContext(MoveTemp(RequestContext), UE::UnifiedError::EDetailFilter::Default);

		SetErrorOnParents(Error.GetValue());
	}
}

void FStreamableHandle::SetErrorOnParents(const UE::UnifiedError::FError& InError)
{
	// Update any meta handles that are still active
	for (TWeakPtr<FStreamableHandle>& WeakHandle : ParentHandles)
	{
		TSharedPtr<FStreamableHandle> Handle = WeakHandle.Pin();

		if (Handle.IsValid())
		{
			Handle->SetError(InError);
		}
	}
}

void FStreamableHandle::AsyncLoadCallbackWrapper(const FName& PackageName, UPackage* Package, EAsyncLoadingResult::Type Result, FSoftObjectPath TargetName)
{
	check(IsInGameThread());

	// Needed so we can bind with a shared pointer for safety
	if (OwningManager)
	{
		TOptional<UE::UnifiedError::FError> MaybeError;

		if (Result != EAsyncLoadingResult::Succeeded)
		{
			MaybeError = ::StreamableManager::GetStreamableError(Result);
			::StreamableManager::FAdditionalContext MoreContext = {PackageName.ToString(), TargetName.ToString()};
			MaybeError->PushErrorContext(MoveTemp(MoreContext), UE::UnifiedError::EDetailFilter::Default);

			SetError(*MaybeError);
		}

		OwningManager->AsyncLoadCallback(MoveTemp(TargetName), Package, MaybeError);

		if (!HasLoadCompleted())
		{
			CallUpdateDelegate();
		}
	}
	else if (!bCanceled)
	{
		UE_LOGF(LogStreamableManager, Verbose, "FStreamableHandle::AsyncLoadCallbackWrapper called on request %ls with result %d with no active manager!", *DebugName, (int32)Result);
	}
}

namespace UE::StreamableManager::Private
{
	// Internal helper for executing delegates. This avoids code duplication by allowing the delegate to be moved or copied when deferred depending on which overload of ExecuteDelegate is called.
	template<typename InStreamableDelegate>
	void ExecuteDelegateInternal(InStreamableDelegate&& Delegate, TSharedPtr<FStreamableHandle> AssociatedHandle, InStreamableDelegate&& CancelDelegate)
	{
		using UnRefCvStreamableDelegate = std::remove_cv_t<std::remove_reference_t<InStreamableDelegate>>;
		static_assert(std::is_same_v<UnRefCvStreamableDelegate, FStreamableDelegateWithHandle>, "ExecuteDelegateInternal is only valid to be called with FStreamableDelegateWithHandle.");

		if (Delegate.IsBound()) // checks for safety as well
		{
			if (GStreamableDelegateDelayFrames == 0)
			{
				// Execute it immediately
				Delegate.Execute(AssociatedHandle);
			}
			else
			{
				// Add to execution queue for next tick
				if (!StreamableDelegateDelayHelper)
				{
					StreamableDelegateDelayHelper = new FStreamableDelegateDelayHelper;
				}

				StreamableDelegateDelayHelper->AddDelegate(Forward<InStreamableDelegate>(Delegate), Forward<InStreamableDelegate>(CancelDelegate), MoveTemp(AssociatedHandle));
			}
		}
	}
}

void FStreamableHandle::ExecuteDelegate(const FStreamableDelegateWithHandle& Delegate, TSharedPtr<FStreamableHandle> AssociatedHandle, const FStreamableDelegateWithHandle& CancelDelegate)
{
	UE::StreamableManager::Private::ExecuteDelegateInternal(Delegate, MoveTemp(AssociatedHandle), CancelDelegate);
}

void FStreamableHandle::ExecuteDelegate(FStreamableDelegateWithHandle&& Delegate, TSharedPtr<FStreamableHandle> AssociatedHandle, FStreamableDelegateWithHandle&& CancelDelegate)
{
	UE::StreamableManager::Private::ExecuteDelegateInternal(MoveTemp(Delegate), MoveTemp(AssociatedHandle), MoveTemp(CancelDelegate));
}

void FStreamableHandle::ExecuteDelegate(const FStreamableDelegate& Delegate, TSharedPtr<FStreamableHandle> AssociatedHandle, const FStreamableDelegate& CancelDelegate)
{
	using namespace UE::StreamableManager::Private;
	ExecuteDelegateInternal(WrapDelegate(Delegate), MoveTemp(AssociatedHandle), WrapDelegate(CancelDelegate));
}

void FStreamableHandle::ExecuteDelegate(FStreamableDelegate&& Delegate, TSharedPtr<FStreamableHandle> AssociatedHandle, FStreamableDelegate&& CancelDelegate)
{
	using namespace UE::StreamableManager::Private;
	ExecuteDelegateInternal(WrapDelegate(MoveTemp(Delegate)), MoveTemp(AssociatedHandle), WrapDelegate(MoveTemp(CancelDelegate)));
}

TSharedPtr<FStreamableHandle> FStreamableHandle::FindMatchingHandle(TFunction<bool(const FStreamableHandle&)> Predicate) const
{
	if (Predicate(*this))
	{
		return ConstCastSharedPtr<FStreamableHandle, const FStreamableHandle, ESPMode::ThreadSafe>(AsShared());
	}

	for (const TSharedPtr<FStreamableHandle>& ChildHandle : ChildHandles)
	{
		if (TSharedPtr<FStreamableHandle> MatchingHandleFromChildren = ChildHandle ? ChildHandle->FindMatchingHandle(Predicate) : nullptr)
		{
			return MatchingHandleFromChildren;
		}
	}

	return nullptr;
}

TSharedPtr<FStreamableHandle> FStreamableHandle::CreateCombinedHandle(const TConstArrayView<TSharedPtr<FStreamableHandle>>& OtherHandles)
{
	static const EStreamableManagerCombinedHandleOptions MergeOptions 
	= EStreamableManagerCombinedHandleOptions::MergeDebugNames | EStreamableManagerCombinedHandleOptions::SkipNulls | EStreamableManagerCombinedHandleOptions::RedirectParents;

	TSharedPtr<FStreamableHandle> ThisAsShared{AsShared()};

	if (FStreamableManager* Manager = GetOwningManager())
	{
		TArray<TSharedPtr<FStreamableHandle>> HandlesToUse{OtherHandles};
		HandlesToUse.AddUnique(ThisAsShared);

		return Manager->CreateCombinedHandle(HandlesToUse, HandleDebugName_CombinedHandle, MergeOptions);
	}

	return nullptr;
}

TSharedPtr<FStreamableHandle> FStreamableHandle::GetOutermostHandle()
{
	if (ParentHandles.Num())
	{
		// have at least one parent handle.

		if (ParentHandles.Num() == 1)
		{
			// can just return the outermost of our one parent
			return ParentHandles[0].Pin()->GetOutermostHandle();
		}

		TArray<TWeakPtr<FStreamableHandle>> IterationArray;
		IterationArray = ParentHandles;
		TArray<TWeakPtr<FStreamableHandle>> NextItrArray;
		bool bFoundParents = true;

		while (bFoundParents)
		{
			for (const TWeakPtr<FStreamableHandle>& WeakHandle : IterationArray)
			{
				TSharedPtr<FStreamableHandle> Handle = WeakHandle.Pin();
				if (Handle.IsValid())
				{
					// Append does the right thing with the allocation already, no need to handle it ourselves
					NextItrArray.Append(Handle->ParentHandles);
				}
			}

			// we didn't find parents if none of our handles we just checked had parent handles
			bFoundParents = NextItrArray.Num() > 0;

			if (!bFoundParents)
			{
				// we can't be in the loop if this array is empty...
				// possible this can actually be more than one handle at this point - that's okay!
				return IterationArray[0].Pin();
			}

			IterationArray = NextItrArray;
			NextItrArray.Reset();
		}
	}
	
	// weren't able to resolve the maximum outermost (likely due to no parent)
	return AsShared();
}

/** Internal object, one of these per object paths managed by this system */
struct FStreamable
{
	/** Hard GC pointer to object */
	TObjectPtr<UObject> Target = nullptr;

	/** Live handles keeping this alive. Handles may be duplicated. */
	TArray<FStreamableHandle*> ActiveHandles;

	/** Handles waiting for this to load. Handles may be duplicated with redirectors. */
	TArray< TSharedRef< FStreamableHandle> > LoadingHandles;

#if UE_ENABLE_STREAMABLE_JIT_ASYNC_LOADING
	/** Id for async load request with async loading system */
	std::atomic<int32> RequestId = INDEX_NONE;
#else
	/** Id for async load request with async loading system */
	int32 RequestId = INDEX_NONE;
#endif // UE_ENABLE_STREAMABLE_JIT_ASYNC_LOADING

	/** If this object is currently being loaded */
	bool bAsyncLoadRequestOutstanding = false;

	/** If this object failed to load, don't try again */
	bool bLoadFailed = false;


	void FreeHandles()
	{
		// Clear the loading handles 
		for (TSharedRef<FStreamableHandle>& LoadingHandle : LoadingHandles)
		{
			LoadingHandle->StreamablesLoading--;
		}
		LoadingHandles.Empty();

		// Cancel active handles, this list includes the loading handles
		while (ActiveHandles.Num() > 0)
		{
			FStreamableHandle* ActiveHandle = ActiveHandles.Pop(EAllowShrinking::No);
			if (!ActiveHandle->bCanceled)
			{
				// Full cancel isn't safe any more

				ActiveHandle->bCanceled = true;
				ActiveHandle->OwningManager = nullptr;

				if (ActiveHandle->bReleased)
				{
					ActiveHandle->NotifyParentsOfCancellation();
				}
				else
				{
					// Keep handle alive to stop the cancel callback from dropping the last reference
					TSharedPtr<FStreamableHandle> SharedHandle = ActiveHandle->AsShared();

					FStreamableHandle::ExecuteDelegate(MoveTemp(ActiveHandle->CancelDelegate), SharedHandle);
					ActiveHandle->UnbindDelegates();
					ActiveHandle->NotifyParentsOfCancellation();
				}
			}
		}
	}

	void AddLoadingRequest(TSharedRef<FStreamableHandle>& NewRequest)
	{
		// With redirectors we can end up adding the same handle multiple times, this is unusual but supported
		ActiveHandles.Add(&NewRequest.Get());

		LoadingHandles.Add(NewRequest);
		NewRequest->StreamablesLoading++;
	}
};

void RemoveActiveHandle(FStreamable& Streamable, FStreamableHandle& Handle)
{
	TArray<FStreamableHandle*>& ActiveHandles = Streamable.ActiveHandles;
	for (int32 Idx = Streamable.ActiveHandles.Num() - 1; Idx >= 0; --Idx)
	{
		if (ActiveHandles[Idx] == &Handle)
		{
			ActiveHandles.RemoveAtSwap(Idx, EAllowShrinking::No);
		}
	}
}
	

FStreamableManager::FStreamableManager()
	: FGCObject(EFlags::AddStableNativeReferencesOnly | EFlags::RegisterLater)
{
#if UE_STREAMABLE_MANAGER_PROFILING_ENABLED
	FStreamableManagerProfiling::TryInitialize();
#endif // UE_STREAMABLE_MANAGER_PROFILING_ENABLED
	FCoreUObjectDelegates::GetPreGarbageCollectDelegate().AddRaw(this, &FStreamableManager::OnPreGarbageCollect);
	bForceSynchronousLoads = false;
	ManagerName = TEXT("StreamableManager");

	// Needs to be called in the most derived constructor to avoid still
	// having pure virtual calls in the virtual table at registration time.
	RegisterGCObject();
}

FStreamableManager::~FStreamableManager()
{
	// Needs to be called in the most derived destructor to avoid leaving
	// a registered object with half its virtual table destroyed.
	UnregisterGCObject();

	FCoreUObjectDelegates::GetPreGarbageCollectDelegate().RemoveAll(this);

	for (const TPair<FSoftObjectPath, FStreamable*>& Pair : StreamableItems)
	{
		Pair.Value->FreeHandles();
	}
	
	TStreamableMap TempStreamables;
	Swap(TempStreamables, StreamableItems);
	for (const TPair<FSoftObjectPath, FStreamable*>& TempPair : TempStreamables)
	{
		delete TempPair.Value;
	}
}

void FStreamableManager::OnPreGarbageCollect()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FStreamableManager::OnPreGarbageCollect);

	TSet<FSoftObjectPath, DefaultKeyFuncs<FSoftObjectPath>, TInlineSetAllocator<256>> RedirectsToRemove;

	const bool bRemoveRedirects = !StreamableRedirects.IsEmpty();

	// Delete and remove inactive streamables
	for (TStreamableMap::TIterator It(StreamableItems); It; ++It)
	{
		FStreamable* Value = It.Value();
		if (Value->ActiveHandles.IsEmpty())
		{
			if (bRemoveRedirects)
			{
				RedirectsToRemove.Add(It.Key());
			}

			Value->FreeHandles();
			delete Value;
			It.RemoveCurrent();
		}
	}

	// Remove redirects
	if (!RedirectsToRemove.IsEmpty())
	{
		for (TStreamableRedirects::TIterator It(StreamableRedirects); It; ++It)
		{
			if (RedirectsToRemove.Contains(It.Value().NewPath))
			{
				It.RemoveCurrent();
			}
		}
	}
}

void FStreamableManager::AddReferencedObjects(FReferenceCollector& Collector)
{
	// If there are active streamable handles in the editor, this will cause the user to Force Delete, which is irritating but necessary because weak pointers cannot be used here
	for (auto& Pair : StreamableItems)
	{
		Collector.AddStableReference(&Pair.Value->Target);
	}
	
	for (TPair<FSoftObjectPath, FRedirectedPath>& Pair : StreamableRedirects)
	{
		Collector.AddStableReference(&Pair.Value.LoadedRedirector);
	}
}

const FString& FStreamableManager::GetManagerName() const
{
	return ManagerName;
}

void FStreamableManager::SetManagerName(FString InName)
{
	ManagerName = MoveTemp(InName);
}

bool FStreamableManager::GetReferencerPropertyName(UObject* Object, FString& OutPropertyName) const
{
	if (ShouldStripDebugName())
	{
		return false;
	}
	if (Object)
	{
		// Report the first handle
		for (TStreamableMap::TConstIterator It(StreamableItems); It; ++It)
		{
			const FStreamable* Existing = It.Value();
			check(Existing);
			if (Existing->Target == Object)
			{
				TStringBuilder<1024> PropName;

				if (Existing->ActiveHandles.Num() > 0)
				{
					PropName.Appendf(TEXT("Active[1/%d]:"), Existing->ActiveHandles.Num());
					PropName << Existing->ActiveHandles[0]->GetDebugName();
					PropName << TEXT(" @ ");
					PropName << UE::SourceLocation::FileAndLine(Existing->ActiveHandles[0]->Location);
				}
				else if (Existing->LoadingHandles.Num() > 0)
				{
					PropName.Appendf(TEXT("Loading[1/%d]:"), Existing->LoadingHandles.Num());
					PropName << Existing->LoadingHandles[0]->GetDebugName();
					PropName << TEXT(" @ ");
					PropName << UE::SourceLocation::FileAndLine(Existing->LoadingHandles[0]->Location);
				}

				if (PropName.Len() > 0)
				{
					OutPropertyName = PropName;
					return true;
				}
			}
		}
	}

	return false;
}

FStreamable* FStreamableManager::FindStreamable(const FSoftObjectPath& Target) const
{
	FStreamable* Existing = StreamableItems.FindRef(Target);

	if (!Existing)
	{
		return StreamableItems.FindRef(ResolveRedirects(Target));
	}

	return Existing;
}

FStreamable* FStreamableManager::StreamInternal(const FSoftObjectPath& InTargetName, TAsyncLoadPriority Priority, TSharedRef<FStreamableHandle> Handle, TSharedRef<IStreamableAsyncLoader> AsyncLoader)
{
	check(IsInGameThread());
	UE_LOGF(LogStreamableManager, Verbose, "Asynchronous load %ls", *InTargetName.ToString());

	FSoftObjectPath TargetName = ResolveRedirects(InTargetName);
	FStreamable* Existing = StreamableItems.FindRef(TargetName);
	if (Existing)
	{
		if (Existing->bAsyncLoadRequestOutstanding)
		{
			UE_LOGF(LogStreamableManager, Verbose, "     Already in progress %ls", *TargetName.ToString());
			check(!Existing->Target); // should not be a load request unless the target is invalid
			check(Existing->RequestId != INDEX_NONE || Existing->ActiveHandles.ContainsByPredicate([](const FStreamableHandle* Handle)
			{
				return Handle->bUseJustInTimeAsyncLoader;
			}));
			ensure(IsAsyncLoading()); // Nothing should be pending if there is no async loading happening

			// Don't return as we potentially want to sync load it
		}
		if (Existing->Target)
		{
			UE_LOGF(LogStreamableManager, Verbose, "     Already Loaded %ls", *TargetName.ToString());
			return Existing;
		}
	}
	else
	{
		Existing = StreamableItems.Add(TargetName, new FStreamable());
	}

	if (!Existing->bAsyncLoadRequestOutstanding)
	{
		FindInMemory(TargetName, Existing);
	}
	

#if !UE_BUILD_SHIPPING

	if (!GStreamableErrorOnRequest.IsEmpty())
	{
		if (FCString::Strstr(*Handle->DebugName, *GStreamableErrorOnRequest) != nullptr)
		{
			TOptional<UE::UnifiedError::FError> Error;
			switch(FMath::Rand() / (RAND_MAX / 4))
			{
			case 0:
				Error = ::StreamableManager::PackageLoadFailed();
				break;
			case 1:
				Error = ::StreamableManager::PackageLoadCanceled();
				break;
			case 2:
				Error = ::StreamableManager::DownloadError();
				break;
			default:
				Error = ::StreamableManager::PackageNameInvalid();
				break;
			}

			Handle->SetError(Error.GetValue());
			Existing->bAsyncLoadRequestOutstanding = false;
			Existing->bLoadFailed = true;
			return Existing;
		}
	}
#endif

	if (!Existing->Target)
	{
		// Disable failed flag as it may have been added at a later point
		Existing->bLoadFailed = false;

		FUObjectThreadContext& ThreadContext = FUObjectThreadContext::Get();

		// If async loading isn't safe or it's forced on, we have to do a sync load which will flush all async loading
		if (GIsInitialLoad || ThreadContext.IsInConstructor > 0 || bForceSynchronousLoads)
		{
			UE_LOGF(LogStreamableManager, Verbose, "     Static loading %ls", *TargetName.ToString());
			Existing->Target = StaticLoadObject(UObject::StaticClass(), nullptr, *TargetName.ToString());

			// Need to manually detect redirectors because the above call only expects to load a UObject::StaticClass() type
			UObjectRedirector* Redir = Cast<UObjectRedirector>(Existing->Target);
			if (Redir)
			{
				TargetName = HandleLoadedRedirector(Redir, TargetName, Existing);
			}

			if (Existing->Target)
			{
				UE_LOGF(LogStreamableManager, Verbose, "     Static loaded %ls", *Existing->Target->GetFullName());
			}
			else
			{
				Existing->bLoadFailed = true;
				UE_LOGF(LogStreamableManager, Log, "Failed attempt to load %ls", *TargetName.ToString());

				Handle->SetError(::StreamableManager::PackageLoadFailed(*TargetName.ToString()));
			}
			Existing->bAsyncLoadRequestOutstanding = false;
		}
		else
		{
			// We always queue a new request in case the existing one gets cancelled
			FString Package = TargetName.ToString();
			int32 FirstDot = Package.Find(TEXT("."), ESearchCase::CaseSensitive);
			if (FirstDot != INDEX_NONE)
			{
				Package.LeftInline(FirstDot,EAllowShrinking::No);
			}

			FPackagePath PackagePath;
			if (!FPackagePath::TryFromPackageName(Package, PackagePath))
			{
				UE_LOGF(LogStreamableManager, Error, "Failed attempt to load %ls; it is not a valid LongPackageName", *Package);
				Existing->bLoadFailed = true;
				Existing->bAsyncLoadRequestOutstanding = false;

				Handle->SetError(::StreamableManager::PackageNameInvalid(Package));
			}
			else
			{
				Existing->bLoadFailed = false;
				Existing->bAsyncLoadRequestOutstanding = true;
#if UE_WITH_PACKAGE_ACCESS_TRACKING
				UE_TRACK_REFERENCING_PACKAGE_SCOPED(Handle->GetReferencerPackage(), Handle->GetRefencerPackageOp());
#endif
#if WITH_EDITOR
				FCookLoadScope CookLoadScope(Handle->GetCookLoadType());
#endif
				uint32 RequestStartTime = FPlatformTime::Cycles();
				TWeakPtr<FStreamableHandle> WeakHandle = Handle;
				FLoadPackageAsyncDelegate LoadPackageAsyncDelegate = FLoadPackageAsyncDelegate::CreateLambda(
					[WeakHandle, TargetName, RequestStartTime](const FName& LoadedPackageName, UPackage* LoadedPackage, EAsyncLoadingResult::Type Result)
					{
#if UE_STREAMABLE_MANAGER_PROFILING_ENABLED
						double SecondsPassed = double(FPlatformTime::Cycles() - RequestStartTime) * FPlatformTime::GetSecondsPerCycle();
						FStreamableManagerProfiling::OnStreamableCompleted(SecondsPassed);
#endif // UE_STREAMABLE_MANAGER_PROFILING_ENABLED
						if (TSharedPtr<FStreamableHandle> Handle = WeakHandle.Pin())
						{
							Handle->AsyncLoadCallbackWrapper(LoadedPackageName, LoadedPackage, Result, TargetName);
						}
					});

				FStreamableAsyncLoaderRequestQueuedDelegate RequestQueuedDelegate = FStreamableAsyncLoaderRequestQueuedDelegate::CreateLambda(
					[WeakHandle, Existing](int32 InRequestID)
					{
#if UE_STREAMABLE_MANAGER_PROFILING_ENABLED
						FStreamableManagerProfiling::OnStreamableQueuedForAsyncLoad();
#endif // UE_STREAMABLE_MANAGER_PROFILING_ENABLED
						if (WeakHandle.IsValid())
						{
							Existing->RequestId = InRequestID;
						}
					});

				AsyncLoader->AddAsyncLoadRequest(
					PackagePath,
					MoveTemp(RequestQueuedDelegate),
					MoveTemp(LoadPackageAsyncDelegate),
					Handle->GetPriority()
				);
			}
		}
	}
	return Existing;
}

void FStreamableManager::CancelJITLoadsRecursively(TSharedRef<FStreamableHandle> Handle)
{
	if (Handle->bUseJustInTimeAsyncLoader)
	{
		// JIT loader does not populate StreamableItem RequestIDs until they are queued. This block here will:
		// 1. Cancel the JIT loader (bUseJustInTimeAsyncLoader = false)
		// 2. If there are streamables not found in memory, will queue async load requests as normal thus populating their RequestIDs (needed in the next step when flushing).
		Handle->bUseJustInTimeAsyncLoader = false;
		for (int32 i = 0; i < Handle->RequestedAssets.Num(); i++)
		{
			// TODO split StreamInternal further so below is not duplicated?
			FSoftObjectPath TargetName = ResolveRedirects(Handle->RequestedAssets[i]);
			FStreamable* Existing = StreamableItems.FindRef(TargetName);
			if (!Existing)
			{
				continue;
			}
			if (Existing->RequestId != INDEX_NONE)
			{
				continue;
			}

			Existing->bAsyncLoadRequestOutstanding = false;
			TSharedRef<FStreamableAsyncLoader> AsyncLoader = MakeShared<FStreamableAsyncLoader>();
			StreamInternal(Handle->RequestedAssets[i], Handle->Priority, Handle, AsyncLoader);
		}
	}

	for (TSharedPtr<FStreamableHandle>& ChildHandle : Handle->ChildHandles)
	{
		CancelJITLoadsRecursively(ChildHandle.ToSharedRef());
	}
}

bool FStreamableManager::ShouldStripDebugName()
{
	bool bShouldStripDebugName = GStreamableStripDebugName;
#if UE_BUILD_SHIPPING
	bShouldStripDebugName = bShouldStripDebugName || GStreamableStripDebugNameInShipping;
#endif
	return bShouldStripDebugName;
}

TSharedPtr<FStreamableHandle> FStreamableManager::RequestAsyncLoad(FStreamableAsyncLoadParams&& Params, FString DebugName, UE::FSourceLocation Location)
{
	LLM_SCOPE(ELLMTag::StreamingManager);

	// Schedule a new callback, this will get called when all related async loads are completed
	TSharedRef<FStreamableHandle> NewRequest = MakeShared<FStreamableHandle>(FStreamableHandle::FPrivateToken(), MoveTemp(Location));
	NewRequest->CompleteDelegate = MoveTemp(Params.OnComplete);
	NewRequest->CancelDelegate = MoveTemp(Params.OnCancel);
	NewRequest->UpdateDelegate = MoveTemp(Params.OnUpdate);
	NewRequest->OwningManager = this;
	NewRequest->RequestedAssets = MoveTemp(Params.TargetsToStream);
#if UE_ENABLE_STREAMABLE_JIT_ASYNC_LOADING
	NewRequest->bUseJustInTimeAsyncLoader = (GStreamableEnableJITAsyncLoading && Params.bUseJustInTimeAsyncLoader) || GStreamableEnableJITAsyncLoadingGlobally;
#endif // UE_ENABLE_STREAMABLE_JIT_ASYNC_LOADING
	if (!ShouldStripDebugName())
	{
		NewRequest->DebugName = MoveTemp(DebugName);
	}

	NewRequest->Priority = Params.Priority;
#if UE_WITH_PACKAGE_ACCESS_TRACKING
	PackageAccessTracking_Private::FTrackedData* AccumulatedScopeData = PackageAccessTracking_Private::FPackageAccessRefScope::GetCurrentThreadAccumulatedData();
	if (AccumulatedScopeData)
	{
		NewRequest->ReferencerPackage = AccumulatedScopeData->PackageName;
		NewRequest->ReferencerPackageOp = AccumulatedScopeData->OpName;
	}
#endif
#if WITH_EDITOR
	NewRequest->CookLoadType = FCookLoadScope::GetCurrentValue();
#endif

	int32 NumValidRequests = NewRequest->RequestedAssets.Num();
	
	TSet<FSoftObjectPath> TargetSet;
	TargetSet.Reserve(NumValidRequests);

#if !UE_BUILD_SHIPPING
	if (!GStreamableDebugOnRequest.IsEmpty())
	{

		for (const FSoftObjectPath& RequestedAsset : NewRequest->RequestedAssets)
		{
			if (RequestedAsset.ToString().Contains(GStreamableDebugOnRequest))
			{
				UE_LOGF(LogStreamableManager, Warning, "Debug on request hit on asset:%ls, debug request:%ls, place breakpoint here to debug", *RequestedAsset.ToString(), *GStreamableDebugOnRequest);
			}
		}
	}
#endif

	for (const FSoftObjectPath& TargetName : NewRequest->RequestedAssets)
	{
		if (TargetName.IsNull())
		{
			--NumValidRequests;
			continue;
		}
		else if (FPackageName::IsShortPackageName(TargetName.GetLongPackageFName()))
		{
			UE_LOGF(LogStreamableManager, Error, "RequestAsyncLoad(%ls) called with invalid package name %ls", *NewRequest->DebugName, *TargetName.ToString());
			NewRequest->CancelDelegate.Unbind(); // cancel delegate is not exepected since a null handle is returned
			NewRequest->CancelHandle();
			return nullptr;
		}
		TargetSet.Add(TargetName);
	}

	if (NumValidRequests == 0)
	{
		// Original array was empty or all null
		UE_LOGF(LogStreamableManager, Display, "RequestAsyncLoad(%ls) called with empty or only null assets!", *NewRequest->DebugName);
		NewRequest->CancelDelegate.Unbind(); // cancel delegate is not exepected since a null handle is returned
		NewRequest->CancelHandle();
		return nullptr;
	} 
	else if (NewRequest->RequestedAssets.Num() != NumValidRequests)
	{
		FString RequestedSet;

		for (auto It = NewRequest->RequestedAssets.CreateIterator(); It; ++It)
		{
			FSoftObjectPath& Asset = *It;
			if (!RequestedSet.IsEmpty())
			{
				RequestedSet += TEXT(", ");
			}
			RequestedSet += Asset.ToString();

			// Remove null entries
			if (Asset.IsNull())
			{
				It.RemoveCurrent();
			}
		}

		// Some valid, some null
		UE_LOGF(LogStreamableManager, Display, "RequestAsyncLoad(%ls) called with both valid and null assets, null assets removed from %ls!", *NewRequest->DebugName, *RequestedSet);
	}

	if (TargetSet.Num() != NewRequest->RequestedAssets.Num())
	{
#if UE_BUILD_DEBUG
		FString RequestedSet;

		for (const FSoftObjectPath& Asset : NewRequest->RequestedAssets)
		{
			if (!RequestedSet.IsEmpty())
			{
				RequestedSet += TEXT(", ");
			}
			RequestedSet += Asset.ToString();
		}

		UE_LOGF(LogStreamableManager, Verbose, "RequestAsyncLoad(%ls) called with duplicate assets, duplicates removed from %ls!", *NewRequest->DebugName, *RequestedSet);
#endif

		NewRequest->RequestedAssets = TargetSet.Array();
	}

	if (Params.bManageActiveHandle)
	{
		// This keeps a reference around until explicitly released
		ManagedActiveHandles.Add(NewRequest);
	}

	if (Params.DownloadParams)
	{
		NewRequest->DownloadCachePin = Params.DownloadParams->CachePin;
		if (!NewRequest->DownloadCachePin)
		{
			NewRequest->DownloadCachePin = FStreamableDownloadCachePin::Create(NewRequest->DebugName);
		}

		NewRequest->DownloadContext = MakeUnique<UE::StreamableManager::Private::FDownloadContext>();
		NewRequest->DownloadContext->Priority = Params.DownloadParams->Priority;
		NewRequest->DownloadContext->bInstallSoftReferences = Params.DownloadParams->bInstallSoftReferences;
		NewRequest->DownloadContext->bAllowMissingDependencies = Params.DownloadParams->bAllowMissingDependencies;
		NewRequest->DownloadContext->bDownloadOnly = Params.DownloadParams->bDownloadOnly;
	}

	if (Params.bStartStalled)
	{
		NewRequest->bStalled = true;
	} 
	else
	{
		StartHandleRequests(NewRequest);
	}

	return NewRequest;
}

TSharedPtr<FStreamableHandle> FStreamableManager::RequestSyncLoadInternal(TArray<FSoftObjectPath>&& TargetsToStream, bool bManageActiveHandle, FString&& DebugName, UE::FSourceLocation&& Location)
{
	FStreamableAsyncLoadParams AsyncParams;
	AsyncParams.TargetsToStream = MoveTemp(TargetsToStream);
	AsyncParams.Priority = AsyncLoadHighPriority;
	AsyncParams.bManageActiveHandle = bManageActiveHandle;

	// Do an async load and wait to complete. In some cases this will do a sync load due to safety issues
	TSharedPtr<FStreamableHandle> Request = RequestAsyncLoad(MoveTemp(AsyncParams), MoveTemp(DebugName), MoveTemp(Location));

	if (Request.IsValid())
	{
		EAsyncPackageState::Type Result = Request->WaitUntilComplete();

		const bool bBadPackageState = !ensureMsgf(
			Result == EAsyncPackageState::Complete,
			TEXT("RequestSyncLoad of %s resulted in bad async load result %d"), *Request->DebugName, Result);

		const bool bLoadNotComplete = !ensureMsgf(
			Request->HasLoadCompleted(),
			TEXT("RequestSyncLoad of %s completed early, not actually completed!"), *Request->DebugName);

		if (bBadPackageState || bLoadNotComplete)
		{
			Request->SetError(::StreamableManager::SyncLoadIncomplete(Request->DebugName));
		}
	}

	return Request;
}

void FStreamableManager::StartHandleRequests(TSharedRef<FStreamableHandle> Handle)
{
	if (Handle->DownloadContext)
	{
		Handle->DownloadContext->Start(Handle);
		// StartHandleRequests will be called again when the download completes
		return;
	}

	Handle->bLoadStarted = true;

	TRACE_LOADTIME_REQUEST_GROUP_SCOPE(TEXT("StreamableManager - %s"), *Handle->GetDebugName());

	TArray<FStreamable *> ExistingStreamables;
	ExistingStreamables.Reserve(Handle->RequestedAssets.Num());

	TSharedPtr<IStreamableAsyncLoader> AsyncLoader;
	if (Handle->bUseJustInTimeAsyncLoader)
	{
		AsyncLoader = MakeShared<FStreamableJustInTimeAsyncLoader>(Handle);
	}
	else
	{
		AsyncLoader = MakeShared<FStreamableAsyncLoader>();
	}

	for (int32 i = 0; i < Handle->RequestedAssets.Num(); i++)
	{
		FStreamable* Existing = StreamInternal(Handle->RequestedAssets[i], Handle->Priority, Handle, AsyncLoader.ToSharedRef());
		check(Existing);

		ExistingStreamables.Add(Existing);
		Existing->AddLoadingRequest(Handle);
	}

	AsyncLoader->StartAsyncLoadRequests();

	// Go through and complete loading anything that's already in memory, this may call the callback right away
	for (int32 i = 0; i < Handle->RequestedAssets.Num(); i++)
	{
		FStreamable* Existing = ExistingStreamables[i];

		if (Existing && (Existing->Target || Existing->bLoadFailed))
		{
			Existing->bAsyncLoadRequestOutstanding = false;

			CheckCompletedRequests(Handle->RequestedAssets[i], Existing);
		}
	}
}

TArray<int32> FStreamableManager::GetAsyncLoadRequestIds(TSharedRef<FStreamableHandle> InitialHandle)
{
	TArray<TSharedPtr<FStreamableHandle>> Queue;
	Queue.Reserve(1 + InitialHandle->ChildHandles.Num());
	Queue.Add(InitialHandle.ToSharedPtr());	
	
	TArray<int32> RequestIds;
	for (int32 i=0; i < Queue.Num(); ++i)
	{
		TSharedPtr<FStreamableHandle> Handle = Queue[i];
		if (!Handle.IsValid()) 
		{
			 continue; 
		}

		Queue.Append(Handle->ChildHandles);	
		
		for (const FSoftObjectPath& Path : Handle->RequestedAssets)
		{
			FStreamable* Streamable = StreamableItems.FindRef(Path);
			if (Streamable && Streamable->RequestId != INDEX_NONE)
			{
				RequestIds.Add(Streamable->RequestId);				
			}
		}
	}
	return RequestIds;
}

UE_TRACE_EVENT_BEGIN(Cpu, StreamableManager_LoadSynchronous, NoSync)
UE_TRACE_EVENT_FIELD(UE::Trace::WideString, AssetPath)
UE_TRACE_EVENT_END()

UObject* FStreamableManager::LoadSynchronous(const FSoftObjectPath& Target, bool bManageActiveHandle, TSharedPtr<FStreamableHandle>* RequestHandlePointer, UE::FSourceLocation Location)
{
#if CPUPROFILERTRACE_ENABLED
	UE_TRACE_LOG_SCOPED_T(Cpu, StreamableManager_LoadSynchronous, CpuChannel)
		<< StreamableManager_LoadSynchronous.AssetPath(*WriteToWideString<FName::StringBufferSize>(Target));
#endif // CPUPROFILERTRACE_ENABLED
	TSharedPtr<FStreamableHandle> Request = RequestSyncLoadInternal(
		TArray<FSoftObjectPath>{Target},
		bManageActiveHandle,
		FString("LoadSynchronous"),
		MoveTemp(Location));

	if (RequestHandlePointer)
	{
		(*RequestHandlePointer) = Request;
	}

	if (Request.IsValid())
	{
		UObject* Result = Request->GetLoadedAsset();

		if (!Result)
		{
			UE_LOGF(LogStreamableManager, Verbose, "LoadSynchronous failed for load of %ls! File is missing or there is a loading system problem", *Target.ToString());
		}

		return Result;
	}

	return nullptr;
}

void FStreamableManager::FindInMemory(FSoftObjectPath& InOutTargetName, struct FStreamable* Existing, UPackage* Package)
{
	check(Existing);
	check(!Existing->bAsyncLoadRequestOutstanding);
	UE_LOGF(LogStreamableManager, Verbose, "     Searching in memory for %ls", *InOutTargetName.ToString());
	Existing->Target = nullptr;

	UObject* Asset = nullptr;
	if (InOutTargetName.GetAssetFName().IsNone())
	{
		if (Package && Package->GetFName() == InOutTargetName.GetLongPackageFName())
		{
			Asset = Package;
		}
		else
		{
			Asset = StaticFindObjectFast(UObject::StaticClass(), nullptr, InOutTargetName.GetLongPackageFName());
		}
	}
	else if (Package)
	{
		Asset = StaticFindObjectFast(UObject::StaticClass(), Package, InOutTargetName.GetAssetFName());
	}
	else
	{
		Asset = StaticFindObject(UObject::StaticClass(), InOutTargetName.GetAssetPath(), EFindObjectFlags::None);
	}

	if (Asset)
	{
		if (InOutTargetName.IsAsset())
		{
			Existing->Target = Asset;
		}
		else if (InOutTargetName.IsSubobject())
		{
			Existing->Target = StaticFindObject(UObject::StaticClass(), Asset, *InOutTargetName.GetSubPathString());
		}
	}
	checkSlow(Existing->Target == StaticFindObject(UObject::StaticClass(), nullptr, *InOutTargetName.ToString()));

	if (Existing->Target && Existing->Target->HasAnyInternalFlags(EInternalObjectFlags_AsyncLoading))
	{
		// This can get called from PostLoad on async loaded objects, if it is we do not want to return partially loaded objects and instead want to register for their full load
		Existing->Target = nullptr;
	}

	UObjectRedirector* Redir = Cast<UObjectRedirector>(Existing->Target);

	if (Redir)
	{
		InOutTargetName = HandleLoadedRedirector(Redir, InOutTargetName, Existing);
	}

	if (Existing->Target)
	{
		UE_LOGF(LogStreamableManager, Verbose, "     Found in memory %ls", *Existing->Target->GetFullName());
		Existing->bLoadFailed = false;
	}
}

void FStreamableManager::AsyncLoadCallback(FSoftObjectPath TargetName, UPackage* Package, const TOptional<UE::UnifiedError::FError> AsyncLoadError)
{
	check(IsInGameThread());

	FStreamable* Existing = FindStreamable(TargetName);

	UE_LOGF(LogStreamableManager, Verbose, "Stream Complete callback %ls", *TargetName.ToString());
	if (Existing)
	{
		const bool bCompleteRequests = Existing->bAsyncLoadRequestOutstanding;

		if (Existing->bAsyncLoadRequestOutstanding)
		{
			Existing->bAsyncLoadRequestOutstanding = false;
			if (!Existing->Target)
			{
				FindInMemory(TargetName, Existing, Package);
			}
		}
		else
		{
			UE_LOGF(LogStreamableManager, Verbose, "AsyncLoadCallback called for %ls when not waiting on a load request, was loaded early by sync load", *TargetName.ToString());
		}

		if (Existing->Target)
		{
			UE_LOGF(LogStreamableManager, Verbose, "    Found target %ls", *Existing->Target->GetFullName());
		}
		else
		{
			// Async load failed to find the object
			Existing->bLoadFailed = true;
			constexpr static const TCHAR LogFmt[] = TEXT("    Failed async load for %s");
			if (GStreamableLogAsyncLoadErrorsAsErrors)
			{
				UE_LOG(LogStreamableManager, Error, LogFmt, *TargetName.ToString());
			}
			else
			{
				UE_LOG(LogStreamableManager, Verbose, LogFmt, *TargetName.ToString());
			}
		}

		if (AsyncLoadError)
		{
			constexpr static const TCHAR LogFmt[] = TEXT("    Encountered error while async loading %s: %s");
			if (GStreamableLogAsyncLoadErrorsAsErrors)
			{
				UE_LOG(LogStreamableManager, Error, LogFmt, *TargetName.ToString(), *AsyncLoadError->GetErrorMessage(true).ToString());
			}
			else
			{
				UE_LOG(LogStreamableManager, Verbose, LogFmt, *TargetName.ToString(), *AsyncLoadError->GetErrorMessage(true).ToString());
			}

			for (const TSharedRef<FStreamableHandle>& Handle : Existing->LoadingHandles)
			{
				Handle->SetError(*AsyncLoadError);
			}
		}
		else if (!Existing->Target)
		{
			for (const TSharedRef<FStreamableHandle>& Handle : Existing->LoadingHandles)
			{
				Handle->SetError(::StreamableManager::PackageLoadFailed(TargetName.ToString()));
			}
		}

		if (bCompleteRequests)
		{
			CheckCompletedRequests(TargetName, Existing);
		}
	}
	else
	{
		// should this be an error?  Is it an error if no handles are waiting for it? 
		UE_LOGF(LogStreamableManager, Error, "Can't find streamable for %ls in AsyncLoadCallback!", *TargetName.ToString());
	}
}

void FStreamableManager::CheckCompletedRequests(const FSoftObjectPath& Target, struct FStreamable* Existing)
{
	static int32 RecursiveCount = 0;

	ensure(RecursiveCount == 0);

	RecursiveCount++;

	// Release these handles at end
	TArray<TSharedRef<FStreamableHandle>> HandlesToComplete;
	TArray<TSharedRef<FStreamableHandle>> HandlesToRelease;

	for (TSharedRef<FStreamableHandle>& Handle : Existing->LoadingHandles)
	{
		ensure(Handle->WasCanceled() || Handle->OwningManager == this);

		// Decrement related requests, and call delegate if all are done and request is still active
		Handle->StreamablesLoading--;
		if (Handle->StreamablesLoading == 0)
		{
			if (Handle->bReleaseWhenLoaded)
			{
				HandlesToRelease.Add(Handle);
			}

			HandlesToComplete.Add(Handle);
		}		
	}
	Existing->LoadingHandles.Empty();

	for (TSharedRef<FStreamableHandle>& Handle : HandlesToComplete)
	{
		Handle->CompleteLoad();
	}

	for (TSharedRef<FStreamableHandle>& Handle : HandlesToRelease)
	{
		Handle->ReleaseHandle();
	}

	// HandlesToRelease might get deleted when function ends

	RecursiveCount--;
}

void FStreamableManager::RemoveReferencedAsset(const FSoftObjectPath& Target, TSharedRef<FStreamableHandle> Handle)
{
	if (Target.IsNull())
	{
		return;
	}

	ensureMsgf(Handle->OwningManager == this, TEXT("RemoveReferencedAsset called on wrong streamable manager for target %s"), *Target.ToString());

	FStreamable* Existing = FindStreamable(Target);

	// This should always be in the active handles list
	if (ensureMsgf(Existing, TEXT("Failed to find existing streamable for %s"), *Target.ToString()))
	{
		ensureMsgf(Existing->ActiveHandles.RemoveSwap(&Handle.Get(), EAllowShrinking::No) > 0, TEXT("Failed to remove active handle for %s"), *Target.ToString());

		// Try removing from loading list if it's still there, this won't call the callback as it's being called from cancel
		// This may remove more than one copy if streamables were merged
		int32 LoadingRemoved = Existing->LoadingHandles.RemoveSwap(Handle);
		if (LoadingRemoved > 0)
		{
			Handle->StreamablesLoading -= LoadingRemoved;

			if (Existing->LoadingHandles.Num() == 0)
			{
				// All requests cancelled, remove loading flag
				Existing->bAsyncLoadRequestOutstanding = false;
			}
		}
	}
}

bool FStreamableManager::AreAllAsyncLoadsComplete() const
{
	check(IsInGameThread());
	for (const TPair<FSoftObjectPath, FStreamable*>& Item : StreamableItems)
	{
		const FStreamable* Streamable = Item.Value;
		if (Streamable && Streamable->bAsyncLoadRequestOutstanding)
		{
			return false;
		}
	}
	return true;
}

bool FStreamableManager::IsAsyncLoadComplete(const FSoftObjectPath& Target) const
{
	// Failed loads count as success
	check(IsInGameThread());
	FStreamable* Existing = FindStreamable(Target);
	return !Existing || !Existing->bAsyncLoadRequestOutstanding;
}

UObject* FStreamableManager::GetStreamed(const FSoftObjectPath& Target) const
{
	check(IsInGameThread());
	FStreamable* Existing = FindStreamable(Target);
	if (Existing && Existing->Target)
	{
		return Existing->Target;
	}
	return nullptr;
}

void FStreamableManager::Unload(const FSoftObjectPath& Target)
{
	check(IsInGameThread());

	TArray<TSharedRef<FStreamableHandle>> HandleList;

	if (GetActiveHandles(Target, HandleList, true))
	{
		for (TSharedRef<FStreamableHandle> Handle : HandleList)
		{
			Handle->ReleaseHandle();
		}
	}
	else
	{
		UE_LOGF(LogStreamableManager, Verbose, "Attempt to unload %ls, but it isn't loaded", *Target.ToString());
	}
}

TSharedPtr<FStreamableHandle> FStreamableManager::CreateCombinedHandle(
	TConstArrayView<TSharedPtr<FStreamableHandle>> ChildHandles,
	FString DebugName,
	EStreamableManagerCombinedHandleOptions Options,
	FStreamableAsyncLoadParams&& Params,
	UE::FSourceLocation Location)
{
	if (ChildHandles.IsEmpty())
	{
		FStreamableHandle::ExecuteDelegate(MoveTemp(Params.OnComplete));
		return nullptr;
	}

	TSharedRef<FStreamableHandle> NewRequest = MakeShared<FStreamableHandle>(FStreamableHandle::FPrivateToken(), MoveTemp(Location));
	NewRequest->OwningManager = this;
	NewRequest->bIsCombinedHandle = true;
	NewRequest->CompleteDelegate = MoveTemp(Params.OnComplete);
	NewRequest->CancelDelegate = MoveTemp(Params.OnCancel);
	NewRequest->UpdateDelegate = MoveTemp(Params.OnUpdate);
	if (!ShouldStripDebugName())
	{
		NewRequest->DebugName = MoveTemp(DebugName);
	}

	static const auto RemapHandleParentRelationship = [](TSharedPtr<FStreamableHandle>& IndividualHandle, TSharedRef<FStreamableHandle>& MergedHandle)
	{
		// break backlinking, inject new forelinking (such that our old parent handles now reference the new MergedHandle - our old parent's ChildHandles array will contain MergedHandle, and MergedHandle is our new Parent)...
		for (int32 Index = 0, Num = IndividualHandle->ParentHandles.Num(); Index < Num; ++Index)
		{
			TSharedPtr<FStreamableHandle> Parent = IndividualHandle->ParentHandles[Index].Pin();

			// even though we should run this check (via invoking this lambda) prior to adding MergedHandle to our ParentHandles, it is still possible if the array is not unique
			if (Parent == MergedHandle)
			{
				continue;
			}

			if (Parent.IsValid())
			{
				const int32 FoundIndex = Parent->ChildHandles.Find(IndividualHandle);
				if (ensure(Parent->ChildHandles.IsValidIndex(FoundIndex)))
				{
					// stomp the old link with the new merged handle
					Parent->ChildHandles[FoundIndex] = MergedHandle;
				}
				else
				{
					// should be impossible, but at least we can add the merged handle for tracking purposes...
					Parent->ChildHandles.Add(MergedHandle);
				}
				
				MergedHandle->ParentHandles.Add(Parent);
			}

			// no longer need the parent handle reference in this handle's parents. The merged handle is within our parent handles, and has us in its ChildHandles.
			IndividualHandle->ParentHandles.RemoveAtSwap(Index);
			Index--;
			Num--;
		}
	};

	TStringBuilder<256> DebugNameBuilder;

	if (EnumHasAllFlags(Options, EStreamableManagerCombinedHandleOptions::MergeDebugNames))
	{
		DebugNameBuilder.Appendf(TEXT("%s: Merged("), *DebugName);
	}

	bool bFormattingFirstPass = true;
	for (TSharedPtr<FStreamableHandle> ChildHandle : ChildHandles)
	{
		if (!ChildHandle.IsValid())
		{
			if (EnumHasAllFlags(Options, EStreamableManagerCombinedHandleOptions::SkipNulls))
			{
				continue;
			}
			else
			{
				ensureMsgf(ChildHandle.IsValid(), TEXT("A child handle is not valid and SkipNulls flag was not used, returning nullptr"));
				return nullptr;
			}
		}

		ensure(ChildHandle->OwningManager == this);

		if (EnumHasAllFlags(Options, EStreamableManagerCombinedHandleOptions::MergeDebugNames))
		{
			if (bFormattingFirstPass)
			{
				DebugNameBuilder.Appendf(TEXT("%s"), *ChildHandle->DebugName);
				bFormattingFirstPass = false;
			}
			else
			{
				DebugNameBuilder.Appendf(TEXT(", %s"), *ChildHandle->DebugName);
			}
		}

		if (EnumHasAllFlags(Options, EStreamableManagerCombinedHandleOptions::RedirectParents))
		{
			RemapHandleParentRelationship(ChildHandle, NewRequest);
		}

		ChildHandle->ParentHandles.Add(NewRequest);
		NewRequest->ChildHandles.Add(ChildHandle);

		if (ChildHandle->bLoadCompleted)
		{
			++NewRequest->CompletedChildCount;
		}
		if (ChildHandle->bCanceled)
		{
			++NewRequest->CanceledChildCount;
		}

		// Propogate any errors to new parent(s)
		if (ChildHandle->Error)
		{
			ChildHandle->SetErrorOnParents(*ChildHandle->Error);
		}
	}

	if (EnumHasAllFlags(Options, EStreamableManagerCombinedHandleOptions::MergeDebugNames))
	{
		DebugNameBuilder += TEXT(")");

		NewRequest->DebugName = *DebugNameBuilder;
	}

	// Add to pending list so these handles don't free when not referenced
	PendingCombinedHandles.Add(NewRequest);

	// This may already be complete
	NewRequest->UpdateCombinedHandle();

	return NewRequest;
}

TSharedPtr<FStreamableHandle> FStreamableManager::CreateErrorHandle(UE::UnifiedError::FError&& Error, FStreamableAsyncLoadParams&& Params, FString DebugName, UE::FSourceLocation Location)
{
	TSharedRef<FStreamableHandle> NewRequest = MakeShared<FStreamableHandle>(FStreamableHandle::FPrivateToken(), MoveTemp(Location));
	NewRequest->OwningManager = this;
	NewRequest->RequestedAssets = MoveTemp(Params.TargetsToStream);
	NewRequest->Priority = Params.Priority;
	if (!ShouldStripDebugName())
	{
		NewRequest->DebugName = MoveTemp(DebugName);
	}

	if (Params.DownloadParams)
	{
		if (!Params.DownloadParams->CachePin)
		{
			Params.DownloadParams->CachePin = FStreamableDownloadCachePin::Create(DebugName);
		}

		NewRequest->DownloadCachePin = Params.DownloadParams->CachePin;
	}

	NewRequest->CancelHandle();
	NewRequest->SetError(MoveTemp(Error));

	return NewRequest;
}

bool FStreamableManager::GetActiveHandles(const FSoftObjectPath& Target, TArray<TSharedRef<FStreamableHandle>>& HandleList, bool bOnlyManagedHandles) const
{
	check(IsInGameThread());
	FStreamable* Existing = FindStreamable(Target);
	if (Existing && Existing->ActiveHandles.Num() > 0)
	{
		for (FStreamableHandle* ActiveHandle : Existing->ActiveHandles)
		{
			ensure(ActiveHandle->OwningManager == this);

			TSharedRef<FStreamableHandle> HandleRef = ActiveHandle->AsShared();
			if (!bOnlyManagedHandles || ManagedActiveHandles.Contains(HandleRef))
			{
				// Only add each handle once, we can have duplicates in the source list
				HandleList.AddUnique(MoveTemp(HandleRef));
			}
		}

		return HandleList.Num() > 0;
	}

	return false;
}

FSoftObjectPath FStreamableManager::ResolveRedirects(const FSoftObjectPath& Target) const
{
	FRedirectedPath const* Redir = StreamableRedirects.Find(Target);
	if (Redir)
	{
		check(Target != Redir->NewPath);
		UE_LOGF(LogStreamableManager, Verbose, "Redirected %ls -> %ls", *Target.ToString(), *Redir->NewPath.ToString());
		return Redir->NewPath;
	}

	// If it wasn't an asset redirect, try applying CoreRedirects
	FSoftObjectPath ReturnPath = Target;
	ReturnPath.FixupCoreRedirects();
	
	return ReturnPath;
}

FSoftObjectPath FStreamableManager::HandleLoadedRedirector(UObjectRedirector* LoadedRedirector, FSoftObjectPath RequestedPath, FStreamable* RequestedStreamable)
{
	UE_LOGF(LogStreamableManager, Verbose, "     Found redirect %ls", *LoadedRedirector->GetPathName());

	// Need to follow the redirector chain
	while (UObjectRedirector* Redirector = Cast<UObjectRedirector>(RequestedStreamable->Target))
	{
		RequestedStreamable->Target = Redirector->DestinationObject;

		if (!RequestedStreamable->Target)
		{
			RequestedStreamable->bLoadFailed = true;
			UE_LOGF(LogStreamableManager, Warning, "Destination of redirector was not found %ls -> %ls.", *RequestedPath.ToString(), *Redirector->GetPathName());

			return RequestedPath;
		}
	}

	FSoftObjectPath NewPath(RequestedStreamable->Target->GetPathName());
	UE_LOGF(LogStreamableManager, Verbose, "     Which redirected to %ls", *NewPath.ToString());

	// We add the originally loaded redirector to GC keep alive, this will keep entire chain alive
	StreamableRedirects.Add(RequestedPath, FRedirectedPath(NewPath, LoadedRedirector));

	// Remove the requested streamable from it's old location
	StreamableItems.Remove(RequestedPath);

	if (FStreamable** FoundStreamable = StreamableItems.Find(NewPath))
	{
		// We found an existing streamable, need to merge the handles and delete old streamable
		// This may result in the same handle being in the list twice! But the rest of the code is ready for that
		// We let LoadFailed and InProgress stay as false on the new streamable because we've successfully loaded an object

		RequestedStreamable->LoadingHandles.Append(MoveTemp((*FoundStreamable)->LoadingHandles));
		RequestedStreamable->ActiveHandles.Append(MoveTemp((*FoundStreamable)->ActiveHandles));

		// No handles remain so need to FreeHandles()
		delete *FoundStreamable;
	}

	// Add requested streamable with new path
	StreamableItems.Add(NewPath, RequestedStreamable);
	
	return NewPath;
}
