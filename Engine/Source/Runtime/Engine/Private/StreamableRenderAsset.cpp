// Copyright Epic Games, Inc. All Rights Reserved.

#include "Engine/StreamableRenderAsset.h"
#include "Components/PrimitiveComponent.h"
#include "Containers/Ticker.h"
#include "RenderAssetUpdate.h"
#include "Rendering/NaniteCoarseMeshStreamingManager.h"
#include "RenderingThread.h"
#include "Streaming/SimpleStreamableAssetManager.h"
#include "AutoRTFM.h"
#include "UObject/Package.h"
#include "UObject/UnrealType.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(StreamableRenderAsset)

const TCHAR* GNoRefBiasQualityLevelCVarName = TEXT("r.Streaming.NoRefLODBiasQualityLevel");
const TCHAR* GNoRefBiasQualityLevelScalabilitySection = TEXT("ViewDistanceQuality");
static int32 GNoRefBiasQualityLevel = -1;
static FAutoConsoleVariableRef CVarNoRefBiasQualityLevel(
	GNoRefBiasQualityLevelCVarName,
	GNoRefBiasQualityLevel,
	TEXT("The quality level for the no-ref mesh streaming LOD bias"),
	ECVF_Scalability);

namespace StreamableRenderAsset
{
#if WITH_EDITOR
	bool bAllowUpdateResourceSize = false;
	FAutoConsoleVariableRef CVarAllowUpdateResourceSize(TEXT("r.Streaming.AllowUpdateResourceSize"), bAllowUpdateResourceSize, TEXT("AllowUpdateResourceSize"));
#endif
}

extern bool TrackRenderAssetEvent(struct FStreamingRenderAsset* StreamingRenderAsset, UStreamableRenderAsset* RenderAsset, bool bForceMipLevelsToBeResident, const FRenderAssetStreamingManager* Manager);

UStreamableRenderAsset::UStreamableRenderAsset(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	check(sizeof(FStreamableRenderResourceState) == sizeof(uint64));

	if (FSimpleStreamableAssetManager::IsEnabled())
	{
		// Pre-publish: no other thread has a reference to this asset yet, so ordering is trivial.
		// The wrapper's operator= performs a release-store (default), which is more than enough.
		SSAMAssetHandle = FSimpleStreamableAssetManager::AllocateAssetHandle();
	}

	SetNoRefStreamingLODBias(-1);
	NoRefStreamingLODBias.SetQualityLevelCVarForCooking(GNoRefBiasQualityLevelCVarName, GNoRefBiasQualityLevelScalabilitySection);
}

UStreamableRenderAsset::~UStreamableRenderAsset() = default;

void UStreamableRenderAsset::RegisterMipLevelChangeCallback(IPrimitiveComponent* Component, int32 LODIndex, float TimeoutSecs, bool bOnStreamIn, FLODStreamingCallback&& Callback)
{
	check(IsInGameThread());

	if (StreamingIndex != INDEX_NONE)
	{
		const int32 ExpectedResidentMips = CachedSRRState.MaxNumLODs - (LODIndex - CachedSRRState.AssetLODBias);
		if (bOnStreamIn == (CachedSRRState.NumResidentLODs >= ExpectedResidentMips))
		{
			Callback(Component, this, ELODStreamingCallbackResult::Success);
			return;
		}

		new (MipChangeCallbacks) FLODStreamingCallbackPayload(Component, FApp::GetCurrentTime() + TimeoutSecs, ExpectedResidentMips, bOnStreamIn, MoveTemp(Callback));
	}
	else
	{
		Callback(Component, this, ELODStreamingCallbackResult::StreamingDisabled);
	}
}

void UStreamableRenderAsset::RegisterMipLevelChangeCallback(IPrimitiveComponent* Component, float TimeoutStartSecs, FLODStreamingCallback&& CallbackStreamingStart, float TimeoutDoneSecs, FLODStreamingCallback&& CallbackStreamingDone)
{
	check(IsInGameThread());

	if (StreamingIndex != INDEX_NONE)
	{
		if (CachedSRRState.NumRequestedLODs > CachedSRRState.NumNonStreamingLODs && CachedSRRState.NumResidentLODs >= CachedSRRState.NumRequestedLODs)
		{
			CallbackStreamingDone(Component, this, ELODStreamingCallbackResult::Success);
			return;
		}

		new (MipChangeCallbacks) FLODStreamingCallbackPayload(Component, FApp::GetCurrentTime() + TimeoutStartSecs, MoveTemp(CallbackStreamingStart), FApp::GetCurrentTime() + TimeoutDoneSecs, MoveTemp(CallbackStreamingDone));
	}
	else
	{
		CallbackStreamingDone(Component, this, ELODStreamingCallbackResult::StreamingDisabled);
	}
}

void UStreamableRenderAsset::RemoveMipLevelChangeCallback(IPrimitiveComponent* Component)
{
	check(IsInGameThread());

	for (int32 Idx = 0; Idx < MipChangeCallbacks.Num(); ++Idx)
	{
		if (MipChangeCallbacks[Idx].Component == Component)
		{
			MipChangeCallbacks[Idx].CallbackDone(Component, this, ELODStreamingCallbackResult::ComponentRemoved);
			MipChangeCallbacks.RemoveAtSwap(Idx--);
		}
	}
}

void UStreamableRenderAsset::RemoveMipLevelChangeCallback(UPrimitiveComponent* Component)
{
	if (Component)
	{
		RemoveMipLevelChangeCallback(Component->GetPrimitiveComponentInterface());
	}
}

void UStreamableRenderAsset::RemoveAllMipLevelChangeCallbacks()
{
	for (int32 Idx = 0; Idx < MipChangeCallbacks.Num(); ++Idx)
	{
		const FLODStreamingCallbackPayload& Payload = MipChangeCallbacks[Idx];
		Payload.CallbackDone(Payload.Component, this, ELODStreamingCallbackResult::AssetRemoved);
	}
	MipChangeCallbacks.Empty();
}

void UStreamableRenderAsset::TickMipLevelChangeCallbacks(TArray<UStreamableRenderAsset*>* DeferredTickCBAssets)
{
	if (MipChangeCallbacks.Num() > 0)
	{
		if (DeferredTickCBAssets)
		{
			DeferredTickCBAssets->Add(this);
			return;
		}

		const double Now = FApp::GetCurrentTime();
		const int32 ResidentMips = CachedSRRState.NumResidentLODs;

		for (int32 Idx = 0; Idx < MipChangeCallbacks.Num(); ++Idx)
		{
			FLODStreamingCallbackPayload& Payload = MipChangeCallbacks[Idx];
			if (PendingUpdate && Payload.CallbackStart)
			{
				Payload.CallbackStart(Payload.Component, this, ELODStreamingCallbackResult::Success);
				Payload.CallbackStart.Reset();
			}
			if (Payload.bIsExpectedResidentMipPayload)
			{
				if (Payload.bOnStreamIn == (ResidentMips >= Payload.ExpectedResidentMips))
				{
					Payload.CallbackDone(Payload.Component, this, ELODStreamingCallbackResult::Success);
					MipChangeCallbacks.RemoveAt(Idx--);
					continue;
				}
			}
			else if (CachedSRRState.NumRequestedLODs != CachedSRRState.NumNonStreamingLODs && ResidentMips >= CachedSRRState.NumRequestedLODs)
			{
				/* this can happen if the streaming was started & done in the same frame. */
				if (Payload.CallbackStart)
				{
					Payload.CallbackStart(Payload.Component, this, ELODStreamingCallbackResult::Success);
				}
				Payload.CallbackDone(Payload.Component, this, ELODStreamingCallbackResult::Success);
				MipChangeCallbacks.RemoveAt(Idx--);
				continue;
			}

			if (Now > Payload.DeadlineStart && Payload.CallbackStart)
			{
				Payload.CallbackStart(Payload.Component, this, ELODStreamingCallbackResult::TimedOut);
				MipChangeCallbacks.RemoveAt(Idx--);
				continue;
			}

			if (Now > Payload.DeadlineDone)
			{
				Payload.CallbackDone(Payload.Component, this, ELODStreamingCallbackResult::TimedOut);
				MipChangeCallbacks.RemoveAt(Idx--);
			}
		}
	}
}

#if WITH_EDITOR
struct FResourceSizeNeedsUpdating
{
	static FResourceSizeNeedsUpdating& Get()
	{
		static FResourceSizeNeedsUpdating Singleton;
		return Singleton;
	}

	void Add(UObject* InObject)
	{
		TWeakObjectPtr<UObject> NewValue(InObject);
		const uint32 NewHash = GetTypeHash(NewValue);

		FScopeLock ScopeLock(&Lock);
		const int32 OriginalNum = Pending.Num();
		Pending.AddByHash(NewHash, MoveTemp(NewValue));

		// Schedule update to occur when not in the middle of a TickStreaming call
		// TickStreaming can be called by multiple threads
		// TickStreaming may also possibly be called where OnObjectPropertyChanged could cause problems
		if (OriginalNum == 0)
		{
			FTSTicker::GetCoreTicker().AddTicker(FTickerDelegate::CreateRaw(this, &FResourceSizeNeedsUpdating::BroadcastOnObjectPropertyChanged));
		}
	}

private:

	bool BroadcastOnObjectPropertyChanged(float DeltaTime)
	{
		check(IsInGameThread());
		if (IsInGameThread())
		{
			FScopeLock ScopeLock(&Lock);
			if (Pending.Num() > 0)
			{
				TRACE_CPUPROFILER_EVENT_SCOPE(FResourceSizeNeedsUpdating::BroadcastOnObjectPropertyChanged);
				FPropertyChangedEvent EmptyPropertyChangedEvent(nullptr);
				for (const TWeakObjectPtr<UObject>& WeakObjectPtr : Pending)
				{
					if (UObject* Obj = WeakObjectPtr.Get())
					{
						// Note: This is too expensive 0.05 to 3 seconds per call, needs to be re-written in a more performance friendly manner
						FCoreUObjectDelegates::OnObjectPropertyChanged.Broadcast(Obj, EmptyPropertyChangedEvent);
					}
				}
				Pending.Empty();
			}
		}

		// Return false because we only wanted one tick
		return false;
	}

	FCriticalSection Lock;
	TSet<TWeakObjectPtr<UObject>> Pending;
};
#endif // WITH_EDITOR

void UStreamableRenderAsset::TickStreaming(ETickStreamingFlags Flags, TArray<UStreamableRenderAsset*>* DeferredTickCBAssets)
{
	// if resident and requested mip counts match then no pending request is in flight
	if (PendingUpdate)
	{
		// When there is no renderthread, allow the gamethread to tick as the renderthread.
		PendingUpdate->Tick(GIsThreadedRendering ? FRenderAssetUpdate::TT_None : FRenderAssetUpdate::TT_Render);

		if (PendingUpdate->IsCompleted())
		{
			if (PendingUpdate->IsSuccessfullyFinished())
			{
				CachedSRRState.NumResidentLODs = CachedSRRState.NumRequestedLODs;
			}
			else
			{
				checkf(PendingUpdate->IsCancelled(), TEXT("Invalid completion of streaming request for asset %s of type %s."), *GetName(), *GetClass()->GetName());
				CachedSRRState.NumRequestedLODs = CachedSRRState.NumResidentLODs;
			}

			PendingUpdate.SafeRelease();

#if WITH_EDITOR
			if (StreamableRenderAsset::bAllowUpdateResourceSize && GIsEditor && EnumHasAnyFlags(Flags, ETickStreamingFlags::SendCompletionEvents))
			{
				// When all the requested mips are streamed in, generate an empty property changed event, to force the
				// ResourceSize asset registry tag to be recalculated.
				FResourceSizeNeedsUpdating::Get().Add(this);
			}
#endif
		}
	}

	if (EnumHasAnyFlags(Flags, ETickStreamingFlags::SendCompletionEvents))
	{
		TickMipLevelChangeCallbacks(DeferredTickCBAssets);
	}
}

void UStreamableRenderAsset::SetForceMipLevelsToBeResident(float Seconds, int32 CinematicLODGroupMask)
{
	const int32 LODGroup = GetLODGroupForStreaming();
	if (CinematicLODGroupMask && LODGroup >= 0 && LODGroup < UE_ARRAY_COUNT(FMath::BitFlag))
	{
		const uint32 TextureGroupBitfield = (uint32)CinematicLODGroupMask;
		bUseCinematicMipLevels = !!(TextureGroupBitfield & FMath::BitFlag[LODGroup]);
	}
	else
	{
		bUseCinematicMipLevels = false;
	}

	ForceMipLevelsToBeResidentTimestamp = FApp::GetCurrentTime() + Seconds;
}

void UStreamableRenderAsset::CancelPendingStreamingRequest()
{
	if (PendingUpdate && !PendingUpdate->IsCancelled())
	{
		PendingUpdate->Abort();
	}
}

bool UStreamableRenderAsset::HasPendingInitOrStreaming() const 
{
	if (!!PendingUpdate)
	{
		return true;
	}

	if (CachedSRRState.IsValid())
	{
		// Avoid a cache miss unless the hint suggests Init could be pending.
		if (CachedSRRState.bHasPendingInitHint)
		{
			if (HasPendingRenderResourceInitialization())
			{
				return true;
			}
			else if (IsInGameThread())
			{
				CachedSRRState.bHasPendingInitHint = false;
			}
		}
		
		return false;
	}

	return HasPendingRenderResourceInitialization();
}

/** Whether there is a pending update and it is locked within an update step. Used to prevent dealocks in SuspendRenderAssetStreaming(). */
bool UStreamableRenderAsset::IsPendingStreamingRequestLocked() const
{
	return PendingUpdate && PendingUpdate->IsLocked();
}

void UStreamableRenderAsset::LinkStreaming()
{
	// Note that this must be called after InitResource() otherwise IsStreamable will always be false.
	EStreamableRenderAssetType RenderAssetType = GetRenderAssetType();
	if (!IsTemplate() && RenderResourceSupportsStreaming() && IStreamingManager::Get().IsRenderAssetStreamingEnabled(RenderAssetType))
	{
		if (StreamingIndex == INDEX_NONE)
		{
			IStreamingManager::Get().GetRenderAssetStreamingManager().AddStreamingRenderAsset(this);
		}
	}
	else
	{
		UnlinkStreaming();
	}
}

void UStreamableRenderAsset::UnlinkStreaming()
{
	if (StreamingIndex != INDEX_NONE)
	{
		IStreamingManager::Get().GetRenderAssetStreamingManager().RemoveStreamingRenderAsset(this);

		// Reset the timer effect from SetForceMipLevelsToBeResident()
		ForceMipLevelsToBeResidentTimestamp = 0;
		// No more streaming events can happen now
		RemoveAllMipLevelChangeCallbacks();
	}
}

bool UStreamableRenderAsset::IsFullyStreamedIn()
{
	// consider a texture fully streamed when it hits this number of LODs :
	//	MaxNumLODs has already been reduced by the "drop mip" LOD Bias
	//	Note that just subtracting off NumCinematicMipLevels is not the right way to get the cinematic lod bias
	//	it should be CalculateLODBias(false) , but we don't have that information here
	int32 FullyStreamedNumLODs = CachedSRRState.MaxNumLODs - NumCinematicMipLevels;

	// Note that if CachedSRRState is not valid, then this asset is not streamable and is then at max resolution.
	if (!CachedSRRState.IsValid() 
		|| !CachedSRRState.bSupportsStreaming 
		|| CachedSRRState.NumResidentLODs >= FullyStreamedNumLODs)
	{
		return true;
	}

#if WITH_EDITOR
	UPackage* Package = GetOutermost();
	if (Package
		&& Package->bIsCookedForEditor
		&& CachedSRRState.NumNonOptionalLODs < CachedSRRState.MaxNumLODs
		&& IStreamingManager::Get().IsRenderAssetStreamingEnabled(EStreamableRenderAssetType::None))
	{
		return IStreamingManager::Get().GetRenderAssetStreamingManager().IsFullyStreamedIn(this);
	}
#endif	

	// IsFullyStreamedIn() might be used incorrectly if any logic waits on it to be true.
	// there could be optional mips which are not available to be loaded, so waiting on IsFullyStreamedIn would never finish
	ensureMsgf(CachedSRRState.NumResidentLODs != CachedSRRState.NumNonOptionalLODs, TEXT("IsFullyStreamedIn() is being called on (%s) which might not have optional LODs mounted."), *GetFName().ToString());

	return false;
}

void UStreamableRenderAsset::WaitForPendingInitOrStreaming(ETickStreamingFlags Flags)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UStreamableRenderAsset::WaitForPendingInitOrStreaming);

	// this can be called with IsAssetStreamingSuspended == true
	//	because Interchange Tasks due to PostEditChange do Texture UpdateResource
	//	those tasks can be retracted in the D3D RHI Wait which runs in the Viewport resize
	//	which turns off streaming
	// @todo : Viewport resize should not turn off streaming

	while (HasPendingInitOrStreaming())
	{
		// Advance the streaming state.
		TickStreaming(Flags);
		// Make sure any render commands are executed, in particular things like InitRHI, or asset updates on the render thread.
		FlushRenderingCommands();

		// Most of the time, sleeping is not required, so avoid loosing a whole quantum (10ms on W10Pro) unless stricly necessary.
		if (HasPendingInitOrStreaming())
		{
			// try to make sure streaming is enabled before doing horrible busy wait
			ensure(!IsAssetStreamingSuspended());

			// Give some time increment so that LOD transition can complete, and also for the gamethread to give room for streaming async tasks.
			FPlatformProcess::Sleep(RENDER_ASSET_STREAMING_SLEEP_DT);
		}
	}
}

void UStreamableRenderAsset::WaitForStreaming(ETickStreamingFlags Flags)
{
	// Complete pending streaming so that the asset can executing new requests if needed.
	WaitForPendingInitOrStreaming(Flags);

	if (IsStreamable())
	{
		// Update the streamer state for this asset and execute new requests if needed. For example force loading to all LODs.
		IStreamingManager::Get().GetRenderAssetStreamingManager().UpdateIndividualRenderAsset(this);
		// Wait for any action to complete.
		WaitForPendingInitOrStreaming(Flags);
	}
}

void UStreamableRenderAsset::BeginDestroy()
{
	Super::BeginDestroy();

	// Abort any pending streaming operation.
	CancelPendingStreamingRequest();

	// Safely unlink the asset from list of streamable.
	UnlinkStreaming();

	// Remove from the list of tracked assets if necessary
	TrackRenderAssetEvent(nullptr, this, false, nullptr);

	if (FSimpleStreamableAssetManager::IsEnabled())
	{
		// Atomically capture-and-clear the SSAM handle and queue it for release. With SSAM
		// enabled the handle is always valid here: the ctor allocates one, and UnregisterAsset
		// preserves the "alive asset has a valid handle" invariant by swapping in a fresh
		// replacement before queueing the old one for release -- so BeginDestroy still has a
		// handle to reclaim even if UnregisterAsset already ran.
		//
		// The atomic Exchange must run from the open (AutoRTFM compiler cannot emit a closed
		// clone over std::atomic ops); OnCommit must be registered from the closed (OnCommit
		// from an open nest runs immediately, defeating the deferral). The reconstruction path
		// snapshots the full UObject memory via RecordOpenWrite before BeginDestroy, so the
		// Exchange is implicitly rolled back on abort. The queue push mutates global SSAM state
		// outside that snapshot, so it must be deferred via OnCommit -- which is a no-op outside
		// transactions and skipped on abort.
		const FSSAMAssetHandle Captured = AutoRTFM::Open([&]
		{
			return SSAMAssetHandle.Exchange(FSSAMAssetHandle{});
		});

		AutoRTFM::OnCommit([Captured]
		{
			FSimpleStreamableAssetManager::QueueRegisteredAssetHandleRelease(Captured);
		});
	}
}

bool UStreamableRenderAsset::IsReadyForFinishDestroy()
{
	if (!Super::IsReadyForFinishDestroy())
	{
		return false;
	}

	if (PendingUpdate)
	{
		bool bCanBeReleased = false;
		if (GEnableAssetUpdateAbandons && PendingUpdate->CanAbandon())
		{
			IStreamingManager::Get().GetRenderAssetStreamingManager().OnRenderAssetUpdateAbandoned(this, PendingUpdate);
			bCanBeReleased = true;
		}
		else if(FRenderAssetUpdateHelper::TickRenderAssetUpdateForGarbageCollection(PendingUpdate))
		{
			bCanBeReleased = true;
		}

		if (bCanBeReleased)
		{
			PendingUpdate.SafeRelease();
		}
	}

	return !PendingUpdate.IsValid();
}

int32 UStreamableRenderAsset::GetCurrentNoRefStreamingLODBias() const
{
	return GetNoRefStreamingLODBias().GetValue(GNoRefBiasQualityLevel);
}

