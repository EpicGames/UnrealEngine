// Copyright Epic Games, Inc. All Rights Reserved.
#include "Sound/StreamedAudioChunkSeekTable.h"

#include "Serialization/MemoryReader.h"
#include "Misc/AssertionMacros.h"
#include "Audio.h"			// LogAudio

#include UE_INLINE_GENERATED_CPP_BY_NAME(StreamedAudioChunkSeekTable)

namespace StreamedAudioChunkSeekTable_Private
{
	static void DeltaEncode(const TArray<uint32>& InValues, TArray<uint16>& OutDeltas)
	{
		if (InValues.Num() > 0)
		{
			OutDeltas.SetNum(InValues.Num());
			for (int32 i = 1; i < InValues.Num(); ++i)
			{
				int32 Delta = InValues[i] - InValues[i - 1];
				OutDeltas[i] = IntCastChecked<uint16>(Delta);
			}
		}
	}
	static void DeltaDecode(const TArray<uint16>& InDeltas, TArray<uint32>& OutValues, uint32 StartingValue = 0)
	{
		OutValues.SetNum(InDeltas.Num());
		uint32 Sum = StartingValue;
		for (int32 i = 0; i < InDeltas.Num(); ++i)
		{
			Sum += InDeltas[i];
			OutValues[i] = Sum;
		}
	}
}

int16 FStreamedAudioChunkSeekTable::GetVersion()
{
	static constexpr int16 Version = 0;		
	return Version;
}

void FStreamedAudioChunkSeekTable::SetMode(EChunkSeekTableMode InMode)
{
	Mode = InMode;
	Impl = CreateImpl(InMode);
}

void FStreamedAudioChunkSeekTable::Reset()
{
	// Will delete and recreate the instance.
	SetMode(Mode);
}

FStreamedAudioChunkSeekTable::FStreamedAudioChunkSeekTable(EChunkSeekTableMode InMode)
{
	SetMode(InMode);
}

bool FStreamedAudioChunkSeekTable::Parse(const uint8* InMemory, uint32 InSize, uint32& OutOffset, FStreamedAudioChunkSeekTable& OutTable)
{
	FMemoryReaderView Reader(MakeArrayView(InMemory, InSize));
	OutTable.Reset();
	if (OutTable.Serialize(Reader))
	{
		int32 Size = Reader.Tell();
		OutOffset += Size;
		UE_LOGF(LogAudio, VeryVerbose, "Successfully parsed seektable: Entries=%d, Size=%d", OutTable.Num(), Size);
		return true;
	}
	return false;
}

uint32 FStreamedAudioChunkSeekTable::GetMagic()
{
	// Magic number is 'SEEK', so it's obvious in memory.
	return (uint32)'KEES';
}

bool FStreamedAudioChunkSeekTable::Serialize(FArchive& Ar)
{
	// For minimizing memory. 
	// Don't save anything if the table is empty, including the magic.
	check(Impl.IsValid());
	if (Ar.IsSaving() && Impl->Num() == 0)
	{
		return true;
	}
	
	// Magic number. (in the case of loading an empty table, this will fail and return false)
	// The outer logic will need to handle that case. 
	uint32 Magic = GetMagic();
	Ar << Magic;
	if (Magic != GetMagic())
	{
		return false;
	}
	
	// Mode.
	uint8 ModeAsByte = static_cast<uint8>(Mode);
	Ar << ModeAsByte;

	if (Ar.IsLoading())
	{
		EChunkSeekTableMode NewMode = static_cast<EChunkSeekTableMode>(ModeAsByte);
		if (NewMode!=Mode)
		{
			SetMode(NewMode);
		}
	}

	check(Impl.IsValid());
	return Impl->Serialize(Ar);
}

struct FConstantRateSeekTable : public FStreamedAudioChunkSeekTable::ISeekTableImpl
{
	TArray<uint32> Table;
	uint32 StartTimeOffset = INDEX_NONE;
	uint16 AudioFramesPerEntry = MAX_uint16;

	static int32 CalcSize(int32 NumEntries)
	{		
		// Not including super class.
		static const int32 SizeOfHeader = sizeof(StartTimeOffset) + sizeof(AudioFramesPerEntry);
		static const int32 SizePerEntry = sizeof(uint16);

		// Header + TArray<uint16> (each entry, plus count).
		return SizeOfHeader + (SizePerEntry * NumEntries) + sizeof(int32);
	}

	int32 Num() const override
	{
		return Table.Num();
	}
	void Add(uint32 InTimeInAudioFrames, uint32 InOffset) override
	{
		if (Table.IsEmpty())
		{
			StartTimeOffset = InTimeInAudioFrames;
		}
		else if(Table.Num() == 1)
		{
			// Imply Samples Per Entry from step 1.
			check(InTimeInAudioFrames > StartTimeOffset);
			int32 DeltaFrames = InTimeInAudioFrames - StartTimeOffset;
			AudioFramesPerEntry = IntCastChecked<uint16>(DeltaFrames);
		}		
		Table.Add(InOffset);
	}

	uint32 FindOffset(uint32 InTimeInAudioFrames) const override
	{
		if (!Table.IsEmpty() && AudioFramesPerEntry > 0)
		{
			int32 Index = (InTimeInAudioFrames - StartTimeOffset) / AudioFramesPerEntry;
			if (Table.IsValidIndex(Index))
			{
				return Table[Index];
			}
		}
		return INDEX_NONE;
	}

	uint32 FindTime(uint32 InOffset) const override
	{
		if (!Table.IsEmpty() && AudioFramesPerEntry > 0 && StartTimeOffset >= 0)
		{
			if (InOffset == 0)
			{
				return StartTimeOffset;
			}
			else
			{
				int32 Index = Table.Find(InOffset);
				if (INDEX_NONE != Index)
				{
					uint32 TimeInSamples = StartTimeOffset + (Index * AudioFramesPerEntry);
					return TimeInSamples;
				}
			}
		}
		return INDEX_NONE;
	}

	bool Serialize(FArchive& Ar) override
	{
		Ar << AudioFramesPerEntry;
		Ar << StartTimeOffset;

		if (Ar.IsSaving())
		{
			Encode(Ar);
		}
		else if (Ar.IsLoading())
		{
			if (!Decode(Ar))
			{
				return false;
			}
		}
		return true;
	}
	void Encode(FArchive& Ar) const
	{
		TArray<uint16> Deltas;
		StreamedAudioChunkSeekTable_Private::DeltaEncode(Table,Deltas);

		// Save in stream.
		Ar << Deltas;
	}
	bool Decode(FArchive& Ar) 
	{
		TArray<uint16> Deltas;
		Ar << Deltas;

		StreamedAudioChunkSeekTable_Private::DeltaDecode(Deltas,Table);
		TArray<uint8> Bytes;
		return true;
	}

	virtual bool GetAt(const uint32 InIndex, uint32& OutOffset, uint32& OutTime) const override
	{
		if (Table.IsValidIndex(InIndex))
		{
			OutTime = AudioFramesPerEntry * InIndex;
			OutOffset = Table[InIndex];
			return true;
		}
		return false;
	}
};

struct FVariableRateSeekTable : public FStreamedAudioChunkSeekTable::ISeekTableImpl
{
	TArray<uint32> Offsets;
	TArray<uint32> Times;

	static int32 CalcSize(int32 NumEntries)
	{
		// Not including super class.
		static const int32 SizeOfHeader = sizeof(uint32);
		static const int32 SizePerEntry = sizeof(uint16) + sizeof(uint16);

		// Header + TArray<uint16> (each entry, plus count).
		return SizeOfHeader + (SizePerEntry * NumEntries) + sizeof(int32) + sizeof(int32);
	}

	void Add(uint32 InTimeInAudioFrames, uint32 InOffset) override
	{
		Offsets.Add(InOffset);
		Times.Add(InTimeInAudioFrames);
	}
	int32 Num() const override
	{
		check(Offsets.Num() == Times.Num());
		return Times.Num();
	}
	uint32 FindOffset(uint32 InTimeInAudioFrames) const override
	{
		check(Offsets.Num() == Times.Num());
		const int32 Index = Algo::UpperBound(Times, InTimeInAudioFrames) - 1;
		if (Offsets.IsValidIndex(Index))
		{
			if (Index > 0)
			{
				return Offsets[Index];
			}
			return Offsets[0];
		}
		return INDEX_NONE;
	}

	uint32 FindTime(uint32 InOffset) const override
	{
		check(Offsets.Num() == Times.Num());
		const int32 Index = Algo::LowerBound(Offsets, InOffset);
		if (Offsets.IsValidIndex(Index))
		{
			return Times[Index];
		}
		return INDEX_NONE;
	}

	bool Serialize(FArchive& Ar) override
	{
		if (Ar.IsSaving())
		{
			Encode(Ar);
		}
		else if (Ar.IsLoading())
		{
			if (!Decode(Ar))
			{
				return false;
			}
		}
		return true;
	}

	void Encode(FArchive& Ar) const
	{
		TArray<uint16> DeltaOffsets;
		StreamedAudioChunkSeekTable_Private::DeltaEncode(Offsets,DeltaOffsets);
		Ar << DeltaOffsets;

		uint32 FirstTimeItem = Times.Num() > 0 ? Times[0] : 0;
		Ar << FirstTimeItem;
		
		TArray<uint16> DeltaTimes;
		StreamedAudioChunkSeekTable_Private::DeltaEncode(Times, DeltaTimes);
		Ar << DeltaTimes;
	}

	bool Decode(FArchive& Ar)
	{
		TArray<uint16> DeltaOffsets;
		Ar << DeltaOffsets;
		StreamedAudioChunkSeekTable_Private::DeltaDecode(DeltaOffsets, Offsets);

		uint32 FirstTimeItem = 0;
		Ar << FirstTimeItem;
		
		TArray<uint16> DeltaTimes;
		Ar << DeltaTimes;
		StreamedAudioChunkSeekTable_Private::DeltaDecode(DeltaTimes, Times, FirstTimeItem);
		return true;
	}

	virtual bool GetAt(const uint32 InIndex, uint32& OutOffset, uint32& OutTime) const override
	{
		if (Offsets.IsValidIndex(InIndex) && Times.IsValidIndex(InIndex))
		{
			OutTime = Times[InIndex];
			OutOffset = Offsets[InIndex];
			return true;
		}
		return false;
	}
};

TUniquePtr<FStreamedAudioChunkSeekTable::ISeekTableImpl> FStreamedAudioChunkSeekTable::CreateImpl(EChunkSeekTableMode InMode)
{
	switch(InMode)
	{
		case EChunkSeekTableMode::ConstantSamplesPerEntry:
			return MakeUnique<FConstantRateSeekTable>();
		case EChunkSeekTableMode::VariableSamplesPerEntry:
			return MakeUnique<FVariableRateSeekTable>();
		default:
			break;
	}
	checkNoEntry();
	return {};
}

int32 FStreamedAudioChunkSeekTable::CalcSize(int32 InNumEntries, EChunkSeekTableMode InMode /*= EChunkSeekTableMode::ConstantSamplesPerEntry*/)
{	
	// Make sure this stays in sync with the Serializer!

	// Don't save anything if the table is empty, including the magic.
	if (InNumEntries == 0)
	{
		return 0;
	}
	
	// Header (magic + mode)
	static const int32 HeaderSize = sizeof(uint32) + sizeof(uint8);	// Magic + mode.

	switch (InMode)
	{
		case EChunkSeekTableMode::ConstantSamplesPerEntry:
			return HeaderSize + FConstantRateSeekTable::CalcSize(InNumEntries);
		case EChunkSeekTableMode::VariableSamplesPerEntry:
			return HeaderSize + FVariableRateSeekTable::CalcSize(InNumEntries);
		default:
			break;
	}
	checkNoEntry();
	return 0;
}

int32 FStreamedAudioChunkSeekTable::CalcSize() const
{
	return CalcSize(Num(), Mode); 
}
