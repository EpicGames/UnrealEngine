// Copyright Epic Games, Inc. All Rights Reserved.

#include "AbandonedRenderAssetUpdateManager.h"
#include "RenderAssetUpdate.h"

#include "Streaming/TextureStreamingHelpers.h"
#include "Engine/StreamableRenderAsset.h"
#include "ProfilingDebugging/CsvProfiler.h"
#include "UObject/UObjectGlobals.h"

CSV_DECLARE_CATEGORY_EXTERN(TextureStreaming);

bool GTickAbandonedRenderAssetUpdatesOnStreamingUpdate = true;
static FAutoConsoleVariableRef CVarTickAbandonedRenderAssetUpdatesOnStreamingUpdate(
	TEXT("r.Streaming.TickAbandonedRenderAssetUpdatesOnStreamingUpdate"),
	GTickAbandonedRenderAssetUpdatesOnStreamingUpdate,
	TEXT("Tick abandoned render asset updates on every streaming update"),
	ECVF_Default
);

FAbandonedRenderAssetUpdateManager::FAbandonedRenderAssetUpdateManager()
{
	FCoreUObjectDelegates::GetPostGarbageCollect().AddRaw(this, &FAbandonedRenderAssetUpdateManager::OnPostGarbageCollect);
}

FAbandonedRenderAssetUpdateManager::~FAbandonedRenderAssetUpdateManager()
{
	UE_CLOGF(!AbandonedRenderAssetUpdates.IsEmpty(),
		LogContentStreaming,
		Warning,
		"FAbandonedRenderAssetUpdateManager shutdown before processing %d render asset updates", AbandonedRenderAssetUpdates.Num());
	FCoreUObjectDelegates::GetPostGarbageCollect().RemoveAll(this);
}

void FAbandonedRenderAssetUpdateManager::OnAbandoned(UStreamableRenderAsset* OwningRenderAsset, TRefCountPtr<class FRenderAssetUpdate> RenderAssetUpdate)
{
	UE_CLOGF(!IsInGameThread(), LogContentStreaming, Fatal, "FAbandonedRenderAssetUpdateManager::OnAbandoned is expected to only be called on the game thread");
	RenderAssetUpdate->OnAbandoned();
	AbandonedRenderAssetUpdates.Add(RenderAssetUpdate);
	UE_LOGF(LogContentStreaming, Log, "FAbandonedRenderAssetUpdateManager Abandoned pending render asset update [Name:%ls]", *OwningRenderAsset->GetName());
}

void FAbandonedRenderAssetUpdateManager::OnPostGarbageCollect()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FAbandonedRenderAssetUpdateManager::OnPostGarbageCollect);
	
	if (GTickAbandonedRenderAssetUpdatesOnStreamingUpdate)
	{
		return;
	}

	TickAbandoned();	
}

void FAbandonedRenderAssetUpdateManager::TickAbandoned()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FAbandonedRenderAssetUpdateManager::TickAbandoned);
	UE_CLOGF(!IsInGameThread(), LogContentStreaming, Fatal, "FAbandonedRenderAssetUpdateManager::TickAbandoned is expected to only tick on the game thread");
	const int32 OutTotal = AbandonedRenderAssetUpdates.Num();
	for (auto It = AbandonedRenderAssetUpdates.CreateIterator(); It; ++It)
	{
		if (FRenderAssetUpdateHelper::TickRenderAssetUpdateForGarbageCollection(*It))
		{
			It.RemoveCurrentSwap();
		}
	}
	const int32 OutCompleted = OutTotal - AbandonedRenderAssetUpdates.Num();
	CSV_CUSTOM_STAT(TextureStreaming, NumAbandonedRenderAssetUpdatesTotal, OutTotal, ECsvCustomStatOp::Set);
	CSV_CUSTOM_STAT(TextureStreaming, NumAbandonedRenderAssetUpdatesCompleted, OutCompleted, ECsvCustomStatOp::Set);
}
