// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
AbandonedRenderAssetUpdateManager.h: Definitions of classes used for handling abandoned render asset updates
=============================================================================*/

#pragma once

#include "CoreMinimal.h"

class UStreamableRenderAsset;
class FRenderAssetUpdate;

extern bool GTickAbandonedRenderAssetUpdatesOnStreamingUpdate;

// The streaming data of a level.
class FAbandonedRenderAssetUpdateManager
{
public:
	FAbandonedRenderAssetUpdateManager();
	~FAbandonedRenderAssetUpdateManager();

	/** Adds a render asset update to the abandoned list to be processed and deallocated independent of the streamable render asset */
	void OnAbandoned(UStreamableRenderAsset* OwningRenderAsset, TRefCountPtr<FRenderAssetUpdate> RenderAssetUpdate);

	/** Tick abandoned render asset updates, called post GC or on each streaming update (see GTickAbandonedRenderAssetUpdatesOnStreamingUpdate) */
	void TickAbandoned();

private:
	/** Internal post garbage collection callbacks to trigger processing of abandoned render asset updates */
	void OnPostGarbageCollect();

private:
	TArray < TRefCountPtr<FRenderAssetUpdate> > AbandonedRenderAssetUpdates;
};
