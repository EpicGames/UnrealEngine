// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	TextureStreamingBuiltDataComponent.cpp
=============================================================================*/

#include "Streaming/ActorTextureStreamingBuildDataComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ActorTextureStreamingBuildDataComponent)

#if WITH_EDITOR

#include "Engine/Texture.h"

void UActorTextureStreamingBuildDataComponent::InitializeTextureStreamingContainer(uint32 InPackedTextureStreamingQualityLevelFeatureLevel)
{
	PackedTextureStreamingQualityLevelFeatureLevel = InPackedTextureStreamingQualityLevelFeatureLevel;
	StreamableTextures.Empty();
}

uint16 UActorTextureStreamingBuildDataComponent::RegisterStreamableTexture(UTexture* InTexture)
{
	const FString& TextureName = InTexture->GetPathName();
	const FGuid& TextureGuid = InTexture->GetLightingGuid();
	
	uint16 Index = 0;
	for (const FStreamableTexture& StreamableTexture : StreamableTextures)
	{
		if (StreamableTexture.Name == TextureName)
		{
			check(StreamableTexture.Guid == TextureGuid);
			return Index;
		}
		++Index;
	}

	StreamableTextures.Emplace(TextureName, TextureGuid);
	return Index;
}

bool UActorTextureStreamingBuildDataComponent::GetStreamableTexture(uint16 InTextureIndex, FString& OutTextureName, FGuid& OutTextureGuid) const
{
	if (StreamableTextures.IsValidIndex(InTextureIndex))
	{
		OutTextureName = StreamableTextures[InTextureIndex].Name;
		OutTextureGuid = StreamableTextures[InTextureIndex].Guid;
		return true;
	}
	return false;
}

uint32 UActorTextureStreamingBuildDataComponent::ComputeHash() const
{
	uint32 Hash = PackedTextureStreamingQualityLevelFeatureLevel;
	for (const FStreamableTexture& StreamableTexture : StreamableTextures)
	{
		Hash = FCrc::TypeCrc32(StreamableTexture.ComputeHash(), Hash);
	}
	return Hash;
}

uint32 FStreamableTexture::ComputeHash() const
{
	return FCrc::TypeCrc32(GetTypeHash(Name), GetTypeHash(Guid));
}

#endif
