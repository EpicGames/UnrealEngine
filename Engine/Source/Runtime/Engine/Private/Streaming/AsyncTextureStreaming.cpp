// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
AsyncTextureStreaming.cpp: Definitions of classes used for texture streaming async task.
=============================================================================*/

#include "Streaming/AsyncTextureStreaming.h"
#include "RHI.h"
#include "Streaming/StreamingManagerTexture.h"
#include "Engine/Level.h"
#include "ProfilingDebugging/CsvProfiler.h"
#include "Streaming/SimpleStreamableAssetManager.h"

#include "ProfilingDebugging/CountersTrace.h"
#include "Streaming/SimpleStreamableAssetManager.h"

#if ALLOW_OTHER_PLATFORM_CONFIG && WITH_EDITOR
#include "DeviceProfiles/DeviceProfileManager.h"
#include "DeviceProfiles/DeviceProfile.h"
#endif

CSV_DECLARE_CATEGORY_EXTERN(TextureStreaming);

TRACE_DECLARE_INT_COUNTER(StreamableAssets, TEXT("StreamableAssets/RegisteredAssets"));

void FAsyncRenderAssetStreamingData::Init(
	TArray<FStreamingViewInfo> InViewInfos,
	float InLastUpdateTime,
	TArray<FLevelRenderAssetManager*>& LevelStaticInstanceManagers,
	FDynamicRenderAssetInstanceManager& DynamicComponentManager)
{
	ViewInfos = InViewInfos;
	LastUpdateTime = InLastUpdateTime;

	DynamicInstancesView = DynamicComponentManager.GetAsyncView(true);

	StaticInstancesViews.Reset();
	StaticInstancesViewIndices.Reset();
	CulledStaticInstancesViewIndices.Reset();
	StaticInstancesViewLevelIndices.Reset();

	for (int32 LevelIndex = 0; LevelIndex<LevelStaticInstanceManagers.Num(); ++LevelIndex)
	{
		if (LevelStaticInstanceManagers[LevelIndex] == nullptr)
		{
			StaticInstancesViewLevelIndices.Add(INDEX_NONE);
			continue;
		}

		const FLevelRenderAssetManager& LevelManager = *LevelStaticInstanceManagers[LevelIndex];

		if (LevelManager.IsInitialized() && LevelManager.GetLevel()->bIsVisible && LevelManager.HasRenderAssetReferences())
		{
			StaticInstancesViewLevelIndices.Add(StaticInstancesViews.Num());
			StaticInstancesViews.Add(LevelStaticInstanceManagers[LevelIndex]->GetAsyncView());
		}
		else
		{
			StaticInstancesViewLevelIndices.Add(INDEX_NONE);
		}
	}
}

void FAsyncRenderAssetStreamingData::Init(
	TArray<FStreamingViewInfo> InViewInfos,
	float InLastUpdateTime)
{
	ViewInfos = InViewInfos;
	LastUpdateTime = InLastUpdateTime;
	
	StaticInstancesViews.Reset();
	StaticInstancesViewIndices.Reset();
	CulledStaticInstancesViewIndices.Reset();
	StaticInstancesViewLevelIndices.Reset();
}

void FAsyncRenderAssetStreamingData::ComputeViewInfoExtras(const FRenderAssetStreamingSettings& Settings)
{
	const float OneOverMaxHiddenPrimitiveViewBoost = 1.f / Settings.MaxHiddenPrimitiveViewBoost;
	const int32 NumViews = ViewInfos.Num();
	ViewInfoExtras.Empty(NumViews);
	ViewInfoExtras.AddUninitialized(NumViews);
	MaxScreenSizeOverAllViews = 0.f;
	
	for (int32 ViewIndex = 0; ViewIndex < NumViews; ++ViewIndex)
	{
		const FStreamingViewInfo& ViewInfo = ViewInfos[ViewIndex];
		FStreamingViewInfoExtra& ViewInfoExtra = ViewInfoExtras[ViewIndex];

		const float EffectiveScreenSize = (Settings.MaxEffectiveScreenSize > 0.0f) ? FMath::Min(Settings.MaxEffectiveScreenSize, ViewInfo.ScreenSize) : ViewInfo.ScreenSize;
		MaxScreenSizeOverAllViews = FMath::Max(MaxScreenSizeOverAllViews, EffectiveScreenSize);
		ViewInfoExtra.ScreenSizeFloat = EffectiveScreenSize * .5f; // Multiply by half since the ratio factors map to half the screen only
		ViewInfoExtra.ExtraBoostForVisiblePrimitiveFloat = 1.f;

		if (ViewInfo.BoostFactor > Settings.MaxHiddenPrimitiveViewBoost)
		{
			ViewInfoExtra.ScreenSizeFloat *= Settings.MaxHiddenPrimitiveViewBoost;
			ViewInfoExtra.ExtraBoostForVisiblePrimitiveFloat = ViewInfo.BoostFactor * OneOverMaxHiddenPrimitiveViewBoost;
		}
		else
		{
			ViewInfoExtra.ScreenSizeFloat *= ViewInfo.BoostFactor;
		}
	}
}

void FAsyncRenderAssetStreamingData::UpdateBoundSizes_Async(const FRenderAssetStreamingSettings& Settings)
{
	for (int32 StaticViewIndex = 0; StaticViewIndex < StaticInstancesViews.Num(); ++StaticViewIndex)
	{
		FRenderAssetInstanceAsyncView& StaticInstancesView = StaticInstancesViews[StaticViewIndex];
		StaticInstancesView.UpdateBoundSizes_Async(ViewInfos, ViewInfoExtras, LastUpdateTime, Settings);

		// Skip levels that can not contribute to resolution.
		if (StaticInstancesView.GetMaxLevelRenderAssetScreenSize() > Settings.MinLevelRenderAssetScreenSize
			|| StaticInstancesView.HasAnyComponentWithForcedLOD())
		{
			StaticInstancesViewIndices.Add(StaticViewIndex);
		}
		else
		{
			CulledStaticInstancesViewIndices.Add(StaticViewIndex);
		}
	}
	if (FSimpleStreamableAssetManager::IsEnabled())
	{
		FSimpleStreamableAssetManager::UpdateBoundSizes(ViewInfos, ViewInfoExtras, LastUpdateTime, Settings);
	}
	
	// Sort by max possible size, this allows early exit when iteration on many levels.
	if (Settings.MinLevelRenderAssetScreenSize > 0)
	{
		StaticInstancesViewIndices.Sort([&](int32 LHS, int32 RHS) { return StaticInstancesViews[LHS].GetMaxLevelRenderAssetScreenSize() > StaticInstancesViews[RHS].GetMaxLevelRenderAssetScreenSize(); });
	}

	DynamicInstancesView.UpdateBoundSizes_Async(ViewInfos, ViewInfoExtras, LastUpdateTime, Settings);
}

void FAsyncRenderAssetStreamingData::UpdatePerfectWantedMips_Async(FStreamingRenderAsset& StreamingRenderAsset, const FRenderAssetStreamingSettings& Settings, bool bOutputToLog) const
{
#if UE_BUILD_SHIPPING
	bOutputToLog = false;
#endif

	// Cache RenderAsset on the stack as it could be nullified on the gamethread.
	const UStreamableRenderAsset* RenderAsset = StreamingRenderAsset.RenderAsset;
	if (!RenderAsset) return;

	float MaxSize = 0;
	float MaxSize_VisibleOnly = 0;
	int32 MaxNumForcedLODs = 0;
	bool bLooksLowRes = false;

	const float MaxAllowedSize = StreamingRenderAsset.GetMaxAllowedSize(MaxScreenSizeOverAllViews);
	const float MaxAssetSize = StreamingRenderAsset.IsMesh() ? MAX_TEXTURE_SIZE : MaxAllowedSize;

#if !UE_BUILD_SHIPPING
	if (Settings.bStressTest)
	{
		// In stress test, we choose between the allowed mips. Combined with "r.Streaming.DropMips=2" this can also generate cancel requests.
		const int32 NumMips = FMath::RandRange(StreamingRenderAsset.MinAllowedMips, StreamingRenderAsset.MaxAllowedMips);
		MaxSize_VisibleOnly = MaxSize = StreamingRenderAsset.GetLODScreenSize(NumMips, MaxScreenSizeOverAllViews);
	}
	else
#endif
	if (Settings.bFullyLoadMeshes && StreamingRenderAsset.IsMesh())
	{
		MaxSize_VisibleOnly = MaxSize = FLT_MAX;
	}
	else if (Settings.bFullyLoadUsedTextures)
	{
		if (StreamingRenderAsset.LastRenderTime < 300 || StreamingRenderAsset.GetFlag(FStreamingRenderAssetFlags::ForceFullyLoad))
		{
			MaxSize_VisibleOnly = FLT_MAX;
		}
	}
	else if (StreamingRenderAsset.MinAllowedMips == StreamingRenderAsset.MaxAllowedMips)
	{
		MaxSize_VisibleOnly = MaxSize = MaxAllowedSize;
	}
	else
	{
		const EStreamableRenderAssetType AssetType = StreamingRenderAsset.RenderAssetType;
		DynamicInstancesView.GetRenderAssetScreenSize(AssetType, RenderAsset, MaxSize, MaxSize_VisibleOnly, MaxNumForcedLODs, MaxAssetSize, bOutputToLog ? TEXT("Dynamic") : nullptr);
		
		if (FSimpleStreamableAssetManager::IsAssetRegistered(StreamingRenderAsset.SSAMAssetHandle))
		{
			// No need to iterate more if render asset is already at maximum resolution.
			const bool bAlreadyAtMaximumResolution = MaxSize_VisibleOnly >= MaxAssetSize || MaxNumForcedLODs >= StreamingRenderAsset.MaxAllowedMips;
			if (!bAlreadyAtMaximumResolution)
			{
				float TmpMaxSize = MaxSize;
				float TmpMaxVisibleOnly = MaxSize_VisibleOnly;
				int32 TmpMaxNumForcedLODs = MaxNumForcedLODs;

				FSimpleStreamableAssetManager::GetRenderAssetScreenSize(AssetType, StreamingRenderAsset.SSAMAssetHandle, TmpMaxSize, TmpMaxVisibleOnly, TmpMaxNumForcedLODs, MaxAssetSize, StreamingRenderAsset.MaxAllowedMips, bOutputToLog ? TEXT("Static") : nullptr);

				MaxSize = FMath::Max(TmpMaxSize, MaxSize);
				MaxSize_VisibleOnly = FMath::Max(TmpMaxVisibleOnly, MaxSize_VisibleOnly);
				MaxNumForcedLODs = FMath::Max(TmpMaxNumForcedLODs, MaxNumForcedLODs);
			}
		}
		
		const bool bAlreadyAtMaximumResolution = MaxSize_VisibleOnly >= MaxAssetSize || MaxNumForcedLODs >= StreamingRenderAsset.MaxAllowedMips;

		bool bCulled = false;
		if (!bAlreadyAtMaximumResolution)
		{
			if (Settings.bMipCalculationEnablePerLevelList)
			{
				int32 LevelToIterateCount = FMath::Min(StreamingRenderAsset.LevelIndexUsage.Num(), StaticInstancesViewLevelIndices.Num());
			
				for (TConstSetBitIterator<> LevelUsageIterator(StreamingRenderAsset.LevelIndexUsage); LevelUsageIterator.GetIndex() < LevelToIterateCount; ++LevelUsageIterator)
				{
					int32 ViewIndex = StaticInstancesViewLevelIndices[LevelUsageIterator.GetIndex()];

					if (ViewIndex==INDEX_NONE)
					{
						continue;
					}

					const FRenderAssetInstanceAsyncView& StaticInstancesView = StaticInstancesViews[ViewIndex];

					if (!StaticInstancesView.HasRenderAssetReferences(RenderAsset))
					{
						// Level entry has been replaced by another level
						// Remove its reference.
						StreamingRenderAsset.LevelIndexUsage[LevelUsageIterator.GetIndex()] = false;
						continue;
					}

					if (StaticInstancesView.GetMaxLevelRenderAssetScreenSize() < Settings.MinLevelRenderAssetScreenSize
						&& !StaticInstancesView.HasComponentWithForcedLOD(RenderAsset))
					{
						bCulled = true;
						continue;
					}

					// No need to iterate more if render asset is already at maximum resolution.
					if (MaxSize_VisibleOnly >= MaxAssetSize || MaxNumForcedLODs >= StreamingRenderAsset.MaxAllowedMips)
					{
						break;
					}

					float TmpMaxSize = MaxSize;
					float TmpMaxVisibleOnly = MaxSize_VisibleOnly;
					int32 TmpMaxNumForcedLODs = MaxNumForcedLODs;
				
					StaticInstancesView.GetRenderAssetScreenSize(AssetType, RenderAsset, TmpMaxSize, TmpMaxVisibleOnly, TmpMaxNumForcedLODs, MaxAssetSize, bOutputToLog ? TEXT("Static") : nullptr);

					MaxSize = FMath::Max(TmpMaxSize, MaxSize);
					MaxSize_VisibleOnly = FMath::Max(TmpMaxVisibleOnly, MaxSize_VisibleOnly);
					MaxNumForcedLODs = FMath::Max(TmpMaxNumForcedLODs, MaxNumForcedLODs);
				}
			}
			else
			{
				for (int32 StaticViewIndex : StaticInstancesViewIndices)
				{
					const FRenderAssetInstanceAsyncView& StaticInstancesView = StaticInstancesViews[StaticViewIndex];

					// No need to iterate more if texture is already at maximum resolution.
					if ((MaxNumForcedLODs >= StreamingRenderAsset.MaxAllowedMips
						|| MaxSize_VisibleOnly >= MAX_TEXTURE_SIZE
						|| (MaxSize_VisibleOnly > StaticInstancesView.GetMaxLevelRenderAssetScreenSize() && Settings.MinLevelRenderAssetScreenSize > 0))
						&& !bOutputToLog)
					{
						break;
					}

					StaticInstancesView.GetRenderAssetScreenSize(AssetType, RenderAsset, MaxSize, MaxSize_VisibleOnly, MaxNumForcedLODs, MaxAssetSize, bOutputToLog ? TEXT("Static") : nullptr);
				}
			}
		}

		// Don't apply to FLT_MAX since it is used as forced streaming. BoostFactor as only meaning for texture/mesh instances since the other heuristics are based on max resolution.
		if (MaxNumForcedLODs < StreamingRenderAsset.MaxAllowedMips
			&& (MaxSize > 0 || MaxSize_VisibleOnly > 0)
			&& MaxSize != FLT_MAX
			&& MaxSize_VisibleOnly != FLT_MAX)
		{
			const float FinalBoostFactor = StreamingRenderAsset.BoostFactor * StreamingRenderAsset.DynamicBoostFactor;

			// If there is not enough resolution in the texture to fix the required quality, save this information to prevent degrading this texture before other ones.
			bLooksLowRes = FMath::Max3(MaxSize_VisibleOnly, MaxSize, MaxAllowedSize) / MaxAllowedSize >= FinalBoostFactor * 2.f;

			MaxSize *= FinalBoostFactor;
			MaxSize_VisibleOnly *= FinalBoostFactor;
		}

		// SSAM removals don't flow through FRenderAssetStreamingManager::SetRenderAssetsRemovedTimestamp
		// the way legacy component/level removals do, so InstanceRemovedTimestamp would sit at its
		// -FLT_MAX default for SSAM-managed assets and bUseUnkownRefHeuristic would fire the instant
		// MaxSize drops to zero, skipping the 5 sec grace period meant to cover transient ref churn
		// (component recreation, level transitions). Edge-detect the "SSAM registered -> not registered"
		// transition here using the handle we already carry, and stamp the timestamp when we observe
		// the flip. The previous-pass bit lives on the streaming asset so SSAM stays unaware of the
		// streaming manager and the handle table doesn't grow a per-slot field that only the asset
		// side would ever use.
		//
		// InstanceRemovedTimestamp is FStreamingRenderAsset::FAtomicTimestamp (relaxed-atomic double)
		// because this write happens on the mip-calc task while legacy GT writers in
		// FRenderAssetStreamingManager can be stamping concurrently. Both writers use
		// FApp::GetCurrentTime(); whichever store wins yields a valid recent-removal time.
		const bool bIsSSAMRegistered = FSimpleStreamableAssetManager::IsAssetRegistered(StreamingRenderAsset.SSAMAssetHandle);
		if (StreamingRenderAsset.GetFlag(FStreamingRenderAssetFlags::WasSSAMRegistered) && !bIsSSAMRegistered)
		{
			StreamingRenderAsset.InstanceRemovedTimestamp = FApp::GetCurrentTime();
		}
		StreamingRenderAsset.SetFlag(FStreamingRenderAssetFlags::WasSSAMRegistered, bIsSSAMRegistered);

		// Last part checks that it has been used since the last reference was removed.
		const float TimeSinceRemoved = (float)(FApp::GetCurrentTime() - StreamingRenderAsset.InstanceRemovedTimestamp);
		bool bUseUnkownRefHeuristic = MaxSize == 0 && MaxSize_VisibleOnly == 0 && !MaxNumForcedLODs && StreamingRenderAsset.LastRenderTime < TimeSinceRemoved - 5.f;
		if (bUseUnkownRefHeuristic)
		{
			if (Settings.bMipCalculationEnablePerLevelList)
			{
				bUseUnkownRefHeuristic = !bCulled;
			}
			else
			{
				// Check that it's not simply culled
				for (int32 StaticViewIndex : CulledStaticInstancesViewIndices)
				{
					const FRenderAssetInstanceAsyncView& StaticInstancesView = StaticInstancesViews[StaticViewIndex];
					if (StaticInstancesView.HasRenderAssetReferences(RenderAsset))
					{
						bUseUnkownRefHeuristic = false;
						break;
					}
				}
			}

			// Ignore bUseUnkownRefHeuristic if they haven't been used in the last 90 sec.
			// If critical, it must be implemented using the ForceFullyLoad logic.
			if (bUseUnkownRefHeuristic && StreamingRenderAsset.LastRenderTime < 90.0f && ((Settings.DropMips != 3 && Settings.DropMips != 4) || AssetType == EStreamableRenderAssetType::Texture))
			{
				if (bOutputToLog) UE_LOGF(LogContentStreaming, Log,  "  UnkownRef");
				MaxSize = FMath::Max(MaxSize, MaxAllowedSize); // affected by HiddenPrimitiveScale
				if (StreamingRenderAsset.LastRenderTime < 5.0f)
				{
					MaxSize_VisibleOnly = FMath::Max<int32>(MaxSize_VisibleOnly, MaxAllowedSize);
				}
			}
		}
		StreamingRenderAsset.SetFlag(FStreamingRenderAssetFlags::UseUnkownRefHeuristic, bUseUnkownRefHeuristic);

		// TODO: for meshes, how to determine whether they are HLOD?
		if (StreamingRenderAsset.GetFlag(FStreamingRenderAssetFlags::ForceFullyLoad)
			|| (AssetType == EStreamableRenderAssetType::Texture
				&& StreamingRenderAsset.LODGroup == TEXTUREGROUP_HierarchicalLOD
				&& Settings.HLODStrategy == 2))
		{
			if (bOutputToLog) UE_LOGF(LogContentStreaming, Log,  "  Forced FullyLoad");
			MaxSize = FLT_MAX; // Forced load ensure the texture gets fully loaded but after what is visible/required by the other logic.
		}
		else if (AssetType == EStreamableRenderAssetType::Texture
			&& StreamingRenderAsset.LODGroup == TEXTUREGROUP_HierarchicalLOD
			&& Settings.HLODStrategy == 1)
		{
			if (bOutputToLog) UE_LOGF(LogContentStreaming, Log,  "  HLOD Strategy");

			if (Settings.bUseNewMetrics)
			{
				MaxSize = FMath::Max(MaxSize, MaxAllowedSize); // Affected by HiddenPrimitiveScale
			}
			else
			{
				MaxSize = FMath::Max(MaxSize, MaxAllowedSize * .5f);
			}
		}
	}

	// Previous metrics didn't handle visibility at all.
	if (!Settings.bUseNewMetrics)
	{
		MaxSize_VisibleOnly = MaxSize = FMath::Max<float>(MaxSize_VisibleOnly, MaxSize);
	}

	StreamingRenderAsset.SetPerfectWantedMips_Async(MaxSize, MaxSize_VisibleOnly, MaxScreenSizeOverAllViews, MaxNumForcedLODs, bLooksLowRes, Settings);
}

bool FRenderAssetStreamingMipCalcTask::AllowPerRenderAssetMipBiasChanges() const
{
	const TArray<FStreamingViewInfo>& ViewInfos = StreamingData.GetViewInfos();
	for (int32 ViewIndex = 0; ViewIndex < ViewInfos.Num(); ++ViewIndex)
	{
		const FStreamingViewInfo& ViewInfo = ViewInfos[ViewIndex];
		if (ViewInfo.BoostFactor > StreamingManager.Settings.PerTextureBiasViewBoostThreshold)
		{
			return false;
		}
	}
	return true;
}

void FRenderAssetStreamingMipCalcTask::ApplyPakStateChanges_Async()
{
	using EOptionalMipsState = FStreamingRenderAsset::EOptionalMipsState;
	using FIoFilenameHashSet = FRenderAssetStreamingManager::FIoFilenameHashSet;

	bool bRecacheAllFiles = false;
	FIoFilenameHashSet MountedStateDirtyFiles;

	// Acquire the pending file state changes from the streaming manager.
	{
		FScopeLock Lock(&StreamingManager.MountedStateDirtyFilesCS);
		Swap(MountedStateDirtyFiles, StreamingManager.MountedStateDirtyFiles);
		Swap(bRecacheAllFiles, StreamingManager.bRecacheAllFiles);
	}

	if (bRecacheAllFiles || MountedStateDirtyFiles.Num())
	{
		for (FStreamingRenderAsset& StreamingRenderAsset : StreamingManager.AsyncUnsafeStreamingRenderAssets)
		{
			if (IsAborted()) break;

			if (StreamingRenderAsset.OptionalFileHash != INVALID_IO_FILENAME_HASH)
			{
				// If there was no valid filename, the hash will be 0, for which DoesMipDataExist() could still change.
				if (bRecacheAllFiles || MountedStateDirtyFiles.Contains(StreamingRenderAsset.OptionalFileHash))
				{
					StreamingRenderAsset.OptionalMipsState = EOptionalMipsState::OMS_NotCached;
				}
			}
		}
	}
}

void FRenderAssetStreamingMipCalcTask::TryDropMaxResolutions(TArray<int32>& PrioritizedRenderAssets, int64& MemoryBudgeted, const int64 InMemoryBudget)
{
	TArray<FStreamingRenderAsset>& StreamingRenderAssets = StreamingManager.AsyncUnsafeStreamingRenderAssets;
	const FRenderAssetStreamingSettings& Settings = StreamingManager.Settings;

	// When using mip bias per texture/mesh, we first reduce the maximum resolutions (if used) in order to fit.
	for (int32 NumDroppedMips = 0; NumDroppedMips < Settings.GlobalMipBias && MemoryBudgeted > InMemoryBudget && !IsAborted(); ++NumDroppedMips)
	{
		const int64 PreviousMemoryBudgeted = MemoryBudgeted;

		// Heuristic: Only consider dropping max resolution for a mesh if it has reasonable impact on memory reduction.
		// Currently, reasonable impact is defined as MemDeltaOfDroppingOneLOD >= MinTextureMemDelta in this pass.
		int64 MinTextureMemDelta = MAX_int64;

		for (int32 PriorityIndex = PrioritizedRenderAssets.Num() - 1; PriorityIndex >= 0 && MemoryBudgeted > InMemoryBudget && !IsAborted(); --PriorityIndex)
		{
			int32 AssetIndex = PrioritizedRenderAssets[PriorityIndex];
			if (AssetIndex == INDEX_NONE) continue;

			FStreamingRenderAsset& StreamingRenderAsset = StreamingRenderAssets[AssetIndex];
			const int32 MinAllowedMips = FMath::Max(StreamingRenderAsset.MinAllowedMips, StreamingRenderAsset.NumForcedMips);
			if (StreamingRenderAsset.BudgetedMips <= MinAllowedMips)
			{
				// Don't try this one again.
				PrioritizedRenderAssets[PriorityIndex] = INDEX_NONE;
				continue;
			}

			// If the texture/mesh requires a high resolution mip, consider dropping it. 
			// When considering dropping the first mip, only textures/meshes using the first mip will drop their resolution, 
			// But when considering dropping the second mip, textures/meshes using their first and second mips will loose it.
			if (StreamingRenderAsset.MaxAllowedMips + StreamingRenderAsset.BudgetMipBias - NumDroppedMips <= StreamingRenderAsset.BudgetedMips)
			{
				const int32 NumMipsToDrop = NumDroppedMips + 1 - StreamingRenderAsset.BudgetMipBias;

				if (Settings.bPrioritizeMeshLODRetention)
				{
					const bool bIsTexture = StreamingRenderAsset.IsTexture();
					const int64 MemDeltaFromMaxResDrop = StreamingRenderAsset.GetDropMaxResMemDelta(NumMipsToDrop);

					if (!MemDeltaFromMaxResDrop || (!bIsTexture && MemDeltaFromMaxResDrop < MinTextureMemDelta && MinTextureMemDelta != MAX_int64))
					{
						continue;
					}

					MinTextureMemDelta = bIsTexture ? FMath::Min(MinTextureMemDelta, MemDeltaFromMaxResDrop) : MinTextureMemDelta;
				}

				MemoryBudgeted -= StreamingRenderAsset.DropMaxResolution_Async(NumMipsToDrop);
			}
		}

		// Break when memory does not change anymore
		if (PreviousMemoryBudgeted == MemoryBudgeted)
		{
			break;
		}
	}
}

void FRenderAssetStreamingMipCalcTask::TryDropMips(TArray<int32>& PrioritizedRenderAssets, int64& MemoryBudgeted, const int64 InMemoryBudget)
{
	TArray<FStreamingRenderAsset>& StreamingRenderAssets = StreamingManager.AsyncUnsafeStreamingRenderAssets;
	const FRenderAssetStreamingSettings& Settings = StreamingManager.Settings;

	while (MemoryBudgeted > InMemoryBudget && !IsAborted())
	{
		const int64 PreviousMemoryBudgeted = MemoryBudgeted;

		// Heuristic: only start considering dropping mesh LODs if it has reasonable impact on memory reduction.
		int64 MinTextureMemDelta = MAX_int64;

		// Drop from the lowest priority first (starting with last elements)
		for (int32 PriorityIndex = PrioritizedRenderAssets.Num() - 1; PriorityIndex >= 0 && MemoryBudgeted > InMemoryBudget && !IsAborted(); --PriorityIndex)
		{
			int32 AssetIndex = PrioritizedRenderAssets[PriorityIndex];
			if (AssetIndex == INDEX_NONE) continue;

			FStreamingRenderAsset& StreamingRenderAsset = StreamingRenderAssets[AssetIndex];
			const int32 MinAllowedMips = FMath::Max(StreamingRenderAsset.MinAllowedMips, StreamingRenderAsset.NumForcedMips);
			if (StreamingRenderAsset.BudgetedMips <= MinAllowedMips)
			{
				// Don't try this one again.
				PrioritizedRenderAssets[PriorityIndex] = INDEX_NONE;
				continue;
			}

			const bool bIsTexture = StreamingRenderAsset.IsTexture();
			const bool bIsMesh = !bIsTexture;
			if (Settings.bPrioritizeMeshLODRetention && bIsMesh)
			{
				const int64 PredictedMemDelta = StreamingRenderAsset.GetDropOneMipMemDelta();
				if (PredictedMemDelta < MinTextureMemDelta && MinTextureMemDelta != MAX_int64)
				{
					continue;
				}
			}

			// If this texture/mesh has already missing mips for its normal quality, don't drop more than required..
			if (StreamingRenderAsset.NumMissingMips > 0)
			{
				--StreamingRenderAsset.NumMissingMips;
				continue;
			}

			const int64 MemDelta = StreamingRenderAsset.DropOneMip_Async();
			MemoryBudgeted -= MemDelta;
			if (Settings.bPrioritizeMeshLODRetention && bIsTexture && MemDelta > 0)
			{
				MinTextureMemDelta = FMath::Min(MinTextureMemDelta, MemDelta);
			}
		}

		// Break when memory does not change anymore
		if (PreviousMemoryBudgeted == MemoryBudgeted)
		{
			break;
		}
	}
}

void FRenderAssetStreamingMipCalcTask::TryKeepMips(TArray<int32>& PrioritizedRenderAssets, int64& MemoryBudgeted, const int64 InMemoryBudget)
{
	TArray<FStreamingRenderAsset>& StreamingRenderAssets = StreamingManager.AsyncUnsafeStreamingRenderAssets;
	bool bBudgetIsChanging = true;
	
	while (MemoryBudgeted < InMemoryBudget && bBudgetIsChanging && !IsAborted())
	{
		bBudgetIsChanging = false;

		// Keep from highest priority first.
		for (int32 PriorityIndex = 0; PriorityIndex < PrioritizedRenderAssets.Num() && MemoryBudgeted < InMemoryBudget && !IsAborted(); ++PriorityIndex)
		{
			int32 AssetIndex = PrioritizedRenderAssets[PriorityIndex];
			if (AssetIndex == INDEX_NONE) continue;

			FStreamingRenderAsset& StreamingRenderAsset = StreamingRenderAssets[AssetIndex];
			int64 TakenMemory = StreamingRenderAsset.KeepOneMip_Async();

			if (TakenMemory > 0)
			{
				if (MemoryBudgeted + TakenMemory <= InMemoryBudget)
				{
					MemoryBudgeted += TakenMemory;
					bBudgetIsChanging = true;
				}
				else // Cancel keeping this mip
				{
					StreamingRenderAsset.DropOneMip_Async();
					PrioritizedRenderAssets[PriorityIndex] = INDEX_NONE; // Don't try this one again.
				}
			}
			else // No other mips to keep.
			{
				PrioritizedRenderAssets[PriorityIndex] = INDEX_NONE; // Don't try this one again.
			}
		}
	}
}

void FRenderAssetStreamingMipCalcTask::UpdateBudgetedMips_Async()
{
	//*************************************
	// Update Budget
	//*************************************

	TArray<FStreamingRenderAsset>& StreamingRenderAssets = StreamingManager.AsyncUnsafeStreamingRenderAssets;
	const FRenderAssetStreamingSettings& Settings = StreamingManager.Settings;

	TArray<int32> PrioritizedRenderAssets;
	TArray<int32> PrioritizedMeshes;

	int32 NumAssets = 0;
	int32 NumMeshes = 0;

	int64 MemoryBudgeted = 0;
	int64 MeshMemoryBudgeted = 0;
	int64 MemoryUsedByNonTextures = 0;
	int64 MemoryUsed = 0;

	for (FStreamingRenderAsset& StreamingRenderAsset : StreamingRenderAssets)
	{
		if (IsAborted()) break;

		if (StreamingRenderAsset.IsVirtualTexture())
		{
			continue;
		}

		const int64 AssetMemBudgeted = StreamingRenderAsset.UpdateRetentionPriority_Async(Settings.bPrioritizeMeshLODRetention);
		const int32 AssetMemUsed = StreamingRenderAsset.GetSize(StreamingRenderAsset.ResidentMips);
		MemoryUsed += AssetMemUsed;

		if (StreamingRenderAsset.IsTexture())
		{
			MemoryBudgeted += AssetMemBudgeted;
			++NumAssets;
		}
		else
		{
			MeshMemoryBudgeted += AssetMemBudgeted;
			MemoryUsedByNonTextures += AssetMemUsed;
			++NumMeshes;
		}
	}

	//*************************************
	// Update Effective Budget
	//*************************************

	bool bResetMipBias = false;

	if (PerfectWantedMipsBudgetResetThresold - MemoryBudgeted - MeshMemoryBudgeted > TempMemoryBudget + MemoryMargin)
	{
		// Reset the budget tradeoffs if the required pool size shrinked significantly.
		PerfectWantedMipsBudgetResetThresold = MemoryBudgeted;
		bResetMipBias = true;
	}
	else if (MemoryBudgeted + MeshMemoryBudgeted > PerfectWantedMipsBudgetResetThresold)
	{
		// Keep increasing the threshold since higher requirements incurs bigger tradeoffs.
		PerfectWantedMipsBudgetResetThresold = MemoryBudgeted + MeshMemoryBudgeted; 
	}


	const int64 NonStreamingRenderAssetMemory =  AllocatedMemory - MemoryUsed + MemoryUsedByNonTextures;
	int64 AvailableMemoryForStreaming = PoolSize - NonStreamingRenderAssetMemory - MemoryMargin;

	// If the platform defines a max VRAM usage, check if the pool size must be reduced,
	// but also check if it would be safe to some of the NonStreamingRenderAssetMemory from the pool size computation.
	// The later helps significantly in low budget settings, where NonStreamingRenderAssetMemory would take too much of the pool.
	if (GPoolSizeVRAMPercentage > 0 && TotalGraphicsMemory > 0)
	{
		const int64 UsableVRAM = FMath::Max<int64>(TotalGraphicsMemory * GPoolSizeVRAMPercentage / 100, TotalGraphicsMemory - Settings.VRAMPercentageClamp * 1024ll * 1024ll);
		const int64 UsedVRAM = (int64)GRHIGlobals.NonStreamingTextureMemorySizeInKB * 1024ll + NonStreamingRenderAssetMemory; // Add any other...
		const int64 AvailableVRAMForStreaming = FMath::Min<int64>(UsableVRAM - UsedVRAM - MemoryMargin, PoolSize);
		if (Settings.bLimitPoolSizeToVRAM || AvailableVRAMForStreaming > AvailableMemoryForStreaming)
		{
			AvailableMemoryForStreaming = AvailableVRAMForStreaming;
		}
	}

	// Update EffectiveStreamingPoolSize, trying to stabilize it independently of temp memory, allocator overhead and non-streaming resources normal variation.
	// It's hard to know how much temp memory and allocator overhead is actually in StreamingMemorySize as it is platform specific.
	// We handle it by not using all memory available. If temp memory and memory margin values are effectively bigger than the actual used values, the pool will stabilize.
	if (AvailableMemoryForStreaming < MemoryBudget)
	{
		// Reduce size immediately to avoid taking more memory.
		MemoryBudget = FMath::Max<int64>(AvailableMemoryForStreaming, 0);
	}
	else if (AvailableMemoryForStreaming - MemoryBudget > TempMemoryBudget + MemoryMargin)
	{
		// Increase size considering that the variation does not come from temp memory or allocator overhead (or other recurring cause).
		// It's unclear how much temp memory is actually in there, but the value will decrease if temp memory increases.
		MemoryBudget = AvailableMemoryForStreaming;
		bResetMipBias = true;
	}

	const int64 PrevMeshMemoryBudget = MeshMemoryBudget;
	MeshMemoryBudget = Settings.MeshPoolSize * 1024 * 1024;
	const bool bUseSeparatePoolForMeshes = MeshMemoryBudget >= 0;

	if (!bUseSeparatePoolForMeshes)
	{
		NumAssets += NumMeshes;
		NumMeshes = 0;
		MemoryBudgeted += MeshMemoryBudgeted;
		MeshMemoryBudgeted = 0;
	}
	else if (PrevMeshMemoryBudget < MeshMemoryBudget)
	{
		bResetMipBias = true;
	}

	//*******************************************
	// Reset per mip bias if not required anymore.
	//*******************************************

	// When using mip per texture/mesh, the BudgetMipBias gets reset when the required resolution does not get affected anymore by the BudgetMipBias.
	// This allows texture/mesh to reset their bias when the viewpoint gets far enough, or the primitive is not visible anymore.
	if (Settings.bUsePerTextureBias)
	{
		for (FStreamingRenderAsset& StreamingRenderAsset : StreamingRenderAssets)
		{
			if (IsAborted()) break;

			if (StreamingRenderAsset.IsVirtualTexture())
			{
				continue;
			}

			if (StreamingRenderAsset.BudgetMipBias > 0
				&& (bResetMipBias
					|| (Settings.bFullyLoadMeshes && StreamingRenderAsset.IsMesh())
					|| FMath::Max<int32>(
						StreamingRenderAsset.VisibleWantedMips,
						StreamingRenderAsset.HiddenWantedMips + StreamingRenderAsset.NumMissingMips) < StreamingRenderAsset.MaxAllowedMips))
			{
				StreamingRenderAsset.BudgetMipBias = 0;
			}
		}
	}

	//*************************************
	// Drop Mips
	//*************************************

	// If the budget is taking too much, drop some mips.
	if ((MemoryBudgeted > MemoryBudget || (bUseSeparatePoolForMeshes && MeshMemoryBudgeted > MeshMemoryBudget)) && !IsAborted())
	{
		//*************************************
		// Get texture/mesh list in order of reduction
		//*************************************

		PrioritizedRenderAssets.Empty(NumAssets);
		PrioritizedMeshes.Empty(NumMeshes);

		for (int32 AssetIndex = 0; AssetIndex < StreamingRenderAssets.Num() && !IsAborted(); ++AssetIndex)
		{
			FStreamingRenderAsset& StreamingRenderAsset = StreamingRenderAssets[AssetIndex];
			// Only consider non deleted textures/meshes (can change any time).
			if (!StreamingRenderAsset.RenderAsset) 
			{
				continue;
			}
			if (StreamingRenderAsset.IsVirtualTexture())
			{
				continue;
			}
			// Ignore textures/meshes for which we are not allowed to reduce resolution.
			if (!StreamingRenderAsset.IsMaxResolutionAffectedByGlobalBias() || (Settings.bFullyLoadMeshes && StreamingRenderAsset.IsMesh()))
			{
				continue;
			}

			// Ignore texture/mesh that can't drop any mips
			const int32 MinAllowedMips = FMath::Max(StreamingRenderAsset.MinAllowedMips, StreamingRenderAsset.NumForcedMips);
			if (StreamingRenderAsset.BudgetedMips > MinAllowedMips)
			{
				if (bUseSeparatePoolForMeshes && StreamingRenderAsset.IsMesh())
				{
					PrioritizedMeshes.Add(AssetIndex);
				}
				else
				{
					PrioritizedRenderAssets.Add(AssetIndex);
				}
			}
		}

		// Sort texture/mesh, having those that should be dropped first.
		PrioritizedRenderAssets.Sort(FCompareRenderAssetByRetentionPriority(StreamingRenderAssets));
		PrioritizedMeshes.Sort(FCompareRenderAssetByRetentionPriority(StreamingRenderAssets));


		if (Settings.bUsePerTextureBias && AllowPerRenderAssetMipBiasChanges())
		{
			//*************************************
			// Drop Max Resolution until in budget.
			//*************************************

			TryDropMaxResolutions(PrioritizedRenderAssets, MemoryBudgeted, MemoryBudget);
			if (bUseSeparatePoolForMeshes)
			{
				TryDropMaxResolutions(PrioritizedMeshes, MeshMemoryBudgeted, MeshMemoryBudget);
			}
		}

		//*************************************
		// Drop WantedMip until in budget.
		//*************************************

		TryDropMips(PrioritizedRenderAssets, MemoryBudgeted, MemoryBudget);
		if (bUseSeparatePoolForMeshes)
		{
			TryDropMips(PrioritizedMeshes, MeshMemoryBudgeted, MeshMemoryBudget);
		}
	}

	//*************************************
	// Keep Mips
	//*************************************

	// If there is some room left, try to keep as much as long as it won't bust budget.
	// This will run even after sacrificing to fit in budget since some small unwanted mips could still be kept.
	if ((MemoryBudgeted < MemoryBudget || (bUseSeparatePoolForMeshes && MeshMemoryBudgeted < MeshMemoryBudget)) && !IsAborted())
	{
		PrioritizedRenderAssets.Empty(NumAssets);
		PrioritizedMeshes.Empty(NumMeshes);

		const int64 MaxAllowedDelta = MemoryBudget - MemoryBudgeted;
		const int64 MaxAllowedMeshDelta = MeshMemoryBudget - MeshMemoryBudgeted;

		for (int32 AssetIndex = 0; AssetIndex < StreamingRenderAssets.Num() && !IsAborted(); ++AssetIndex)
		{
			FStreamingRenderAsset& StreamingRenderAsset = StreamingRenderAssets[AssetIndex];
			// Only consider non deleted textures/meshes (can change any time).
			if (!StreamingRenderAsset.RenderAsset) 
			{
				continue;
			}
			if (StreamingRenderAsset.IsVirtualTexture())
			{
				continue;
			}
			// Only consider textures/meshes that won't bust budget nor generate new I/O requests
			if (StreamingRenderAsset.BudgetedMips < StreamingRenderAsset.ResidentMips)
			{
				const int32 Delta = StreamingRenderAsset.GetSize(StreamingRenderAsset.BudgetedMips + 1) - StreamingRenderAsset.GetSize(StreamingRenderAsset.BudgetedMips);
				const bool bUseMeshVariant = bUseSeparatePoolForMeshes && StreamingRenderAsset.IsMesh();
				const int64 MaxDelta = bUseMeshVariant ? MaxAllowedMeshDelta : MaxAllowedDelta;
				TArray<int32>& AssetIndcies = bUseMeshVariant ? PrioritizedMeshes : PrioritizedRenderAssets;

				if (Delta <= MaxDelta)
				{
					AssetIndcies.Add(AssetIndex);
				}
			}
		}

		// Sort texture/mesh, having those that should be dropped first.
		PrioritizedRenderAssets.Sort(FCompareRenderAssetByRetentionPriority(StreamingRenderAssets));
		PrioritizedMeshes.Sort(FCompareRenderAssetByRetentionPriority(StreamingRenderAssets));

		TryKeepMips(PrioritizedRenderAssets, MemoryBudgeted, MemoryBudget);
		if (bUseSeparatePoolForMeshes)
		{
			TryKeepMips(PrioritizedMeshes, MeshMemoryBudgeted, MeshMemoryBudget);
		}
	}

	//*************************************
	// Handle drop mips debug option
	//*************************************
#if !UE_BUILD_SHIPPING
	if (Settings.DropMips > 0 && Settings.DropMips != 4)
	{
		for (FStreamingRenderAsset& StreamingRenderAsset : StreamingRenderAssets)
		{
			if (IsAborted()) break;

			if (Settings.DropMips == 1)
			{
				StreamingRenderAsset.BudgetedMips = FMath::Min<int32>(StreamingRenderAsset.BudgetedMips, StreamingRenderAsset.GetPerfectWantedMips());
			}
			else
			{
				StreamingRenderAsset.BudgetedMips = FMath::Min<int32>(StreamingRenderAsset.BudgetedMips, StreamingRenderAsset.VisibleWantedMips);
			}
		}
	}
#endif
}

void FRenderAssetStreamingMipCalcTask::UpdateBudgetedVirtualTextureMips_Async()
{
	TArray<FStreamingRenderAsset>& StreamingRenderAssets = StreamingManager.AsyncUnsafeStreamingRenderAssets;
	const FRenderAssetStreamingSettings& Settings = StreamingManager.Settings;
	int64 TargetMemoryBudget = FMath::Max(Settings.VirtualTexturePoolSize * 1024 * 1024, 0);
	const bool bDisableStreaming = TargetMemoryBudget == 0;

	int64 MemoryBudgeted = 0;
	TArray<int32> AssetIndices;

	// Iterate the virtual texture streaming assets, set their current budget, and collect candidates for budget reduction.
	for (int32 AssetIndex = 0; AssetIndex < StreamingRenderAssets.Num() && !IsAborted(); ++AssetIndex)
	{
		FStreamingRenderAsset& StreamingRenderAsset = StreamingRenderAssets[AssetIndex];
		if (StreamingRenderAsset.RenderAsset && StreamingRenderAsset.IsVirtualTexture())
		{
			if (bDisableStreaming)
			{
				StreamingRenderAsset.BudgetedMips = StreamingRenderAsset.MinAllowedMips;
			}
			else
			{
				MemoryBudgeted += StreamingRenderAsset.UpdateRetentionPriority_Async(Settings.bPrioritizeMeshLODRetention);
				// Don't consider the virtual texture root pages as part of our budget.
				MemoryBudgeted -= StreamingRenderAsset.GetSize(StreamingRenderAsset.MinAllowedMips);

				const int32 MinAllowedMips = FMath::Max(StreamingRenderAsset.MinAllowedMips, StreamingRenderAsset.NumForcedMips);
				if (StreamingRenderAsset.BudgetedMips > MinAllowedMips)
				{
					AssetIndices.Add(AssetIndex);
				}
			}
		}
	}

	// Try to drop mips when overbudget.
	if (MemoryBudgeted > TargetMemoryBudget && !IsAborted())
	{
		// Sort to prioritize mip retention.
		AssetIndices.Sort(FCompareRenderAssetByRetentionPriority(StreamingRenderAssets));
		// Drop mips to reach target budget.
		TryDropMips(AssetIndices, MemoryBudgeted, TargetMemoryBudget);
	}
}

void FRenderAssetStreamingMipCalcTask::UpdateLoadAndCancelationRequests_Async()
{
	TArray<FStreamingRenderAsset>& StreamingRenderAssets = StreamingManager.AsyncUnsafeStreamingRenderAssets;
	const FRenderAssetStreamingSettings& Settings = StreamingManager.Settings;

	CancelationRequests.Empty();

	int64 StreamOutMemoryBudget = TempMemoryBudget;
	int64 StreamInMemoryBudget = TempMemoryBudget;

	TArray<int32> PrioritizedRenderAssets;
	if (!IsAborted())
	{
		PrioritizedRenderAssets.Empty(StreamingRenderAssets.Num());
	}
	for (int32 AssetIndex = 0; AssetIndex < StreamingRenderAssets.Num() && !IsAborted(); ++AssetIndex)
	{
		FStreamingRenderAsset& StreamingRenderAsset = StreamingRenderAssets[AssetIndex];
		const bool bWasMissingTooManyMips = StreamingRenderAsset.IsMissingTooManyMips();

		// If we need to change the number of resident mips.
		if (StreamingRenderAsset.UpdateLoadOrderPriority_Async(Settings))
		{
			// If there is no pending update, kick one if the budget allows it.
			if (StreamingRenderAsset.RequestedMips == StreamingRenderAsset.ResidentMips)
			{
				PrioritizedRenderAssets.Add(AssetIndex);
			}
			// Otherwise, if the update is trying to load too many, too few, or unload required MIPs, (try to) cancel it.
			else if (
				// If marked as missing too many MIPs, a high priority request was created so be more aggressive on canceling it.
				StreamingRenderAsset.RequestedMips > FMath::Max<int32>(StreamingRenderAsset.ResidentMips, StreamingRenderAsset.WantedMips + (bWasMissingTooManyMips ? 0 : 1)) ||
				// If too many missing MIPs, cancel existing request if it is not loading enough so a high priority one can be created.
				// Otherwise, only cancel if it is trying to unload resident MIPs.
				StreamingRenderAsset.RequestedMips < (StreamingRenderAsset.IsMissingTooManyMips() ? StreamingRenderAsset.WantedMips : FMath::Min<int32>(StreamingRenderAsset.ResidentMips, StreamingRenderAsset.WantedMips)))
			{
				CancelationRequests.Add(AssetIndex);
			}
		}

		// Reduce the stream in/out budgets from pending updates.
		const int64 TempMemoryUsed = StreamingRenderAsset.GetSize(StreamingRenderAsset.RequestedMips);
		if (StreamingRenderAsset.RequestedMips < StreamingRenderAsset.ResidentMips)
		{
			// Here we assume that the stream out complete before new stream in requests start, so it doesn't affect stream in budget.
			StreamOutMemoryBudget -= TempMemoryUsed;
		}
		else if (StreamingRenderAsset.RequestedMips > StreamingRenderAsset.ResidentMips)
		{
			// If there is a pending stream in, remove the temporary memory from both stream in and stream out budget.
			// When the request was made, there were possibly stream out issued at the same time to free memory in case of budget limit.
			StreamInMemoryBudget -= TempMemoryUsed;
			StreamOutMemoryBudget -= TempMemoryUsed;
		}
	}

	PrioritizedRenderAssets.Sort(FCompareRenderAssetByLoadOrderPriority(StreamingRenderAssets));

	// If possible, free as much memory with stream out operations, as will be required with new stream in requests.
	// This prevents becoming overbudget momentarily, when we are already at the budget limit.
	TArray<int32> StreamOutRequests;
	TArray<int32> StreamInRequests;

	if (!IsAborted())
	{
		StreamOutRequests.Reserve(PrioritizedRenderAssets.Num());
		StreamInRequests.Reserve(PrioritizedRenderAssets.Num());
	}

	// Now fill in the LoadRequest and CancelationRequests
	for (int32 PriorityIndex = 0; PriorityIndex < PrioritizedRenderAssets.Num() && !IsAborted(); ++PriorityIndex)
	{
		int32 AssetIndex = PrioritizedRenderAssets[PriorityIndex];
		FStreamingRenderAsset& StreamingRenderAsset = StreamingRenderAssets[AssetIndex];
		// This assumes that the assets are streamed through a copy of all LODs.
		// Even though this is only the case for non partially resident textures, 
		// we still use this metric to limit the number of pending streaming requests.
		const int64 TempMemoryRequired = StreamingRenderAsset.GetSize(StreamingRenderAsset.WantedMips);

		// Check whether the budget allows the update, with the exception of always allowing a single update of any size (otherwise completion might never happen).
		if (StreamingRenderAsset.WantedMips < StreamingRenderAsset.ResidentMips && (TempMemoryRequired <= StreamOutMemoryBudget || !StreamOutRequests.Num()))
		{
			StreamOutRequests.Add(AssetIndex);
			StreamOutMemoryBudget -= TempMemoryRequired;
		}
		else if (StreamingRenderAsset.WantedMips > StreamingRenderAsset.ResidentMips && (TempMemoryRequired <= StreamInMemoryBudget || !StreamInRequests.Num()))
		{
			StreamInRequests.Add(AssetIndex);
			StreamInMemoryBudget -= TempMemoryRequired;
		}
	}

	LoadRequests.Reset(StreamOutRequests.Num() + StreamInRequests.Num());
	// Process stream out requests first since they execute faster, freeing the memory for the stream in requests.
	LoadRequests.Append(StreamOutRequests);
	LoadRequests.Append(StreamInRequests);
}

void FRenderAssetStreamingMipCalcTask::UpdatePendingStreamingStatus_Async()
{
	TArray<FStreamingRenderAsset>& StreamingRenderAssets = StreamingManager.AsyncUnsafeStreamingRenderAssets;
	const bool bIsStreamingPaused = StreamingManager.bPauseRenderAssetStreaming;

	PendingUpdateDirties.Empty();

	for (int32 AssetIndex = 0; AssetIndex < StreamingRenderAssets.Num() && !IsAborted(); ++AssetIndex)
	{
		const FStreamingRenderAsset& StreamingTexture = StreamingRenderAssets[AssetIndex];
		if (StreamingTexture.GetFlag(FStreamingRenderAssetFlags::HasUpdatePending) != StreamingTexture.HasUpdatePending(bIsStreamingPaused, HasAnyView()))
		{
			// The texture/mesh state are only updated on the gamethread, where we can make sure the UStreamableRenderAsset is in sync.
			PendingUpdateDirties.Add(AssetIndex);
		}
	}
}

void FRenderAssetStreamingMipCalcTask::DoWork()
{
	SCOPED_NAMED_EVENT(FRenderAssetStreamingMipCalcTask_DoWork, FColor::Turquoise);
	DECLARE_SCOPE_CYCLE_COUNTER(TEXT("FRenderAssetStreamingMipCalcTask::DoWork"), STAT_FRenderAssetStreamingMipCalcTask_DoWork, STATGROUP_StreamingDetails);

	// While the async task is runnning, the StreamingRenderAssets are guarantied not to be reallocated.
	// 2 things can happen : a texture can be removed, in which case the texture will be set to null
	// or some members can be updated following calls to UpdateDynamicData().
	TArray<FStreamingRenderAsset>& StreamingRenderAssets = StreamingManager.AsyncUnsafeStreamingRenderAssets;
	const FRenderAssetStreamingSettings& Settings = StreamingManager.Settings;

	StreamingData.ComputeViewInfoExtras(Settings);

	FSimpleStreamableAssetManager::FScopedLock ScopedLock(FSimpleStreamableAssetManager::GetCriticalSection(), FSimpleStreamableAssetManager::IsEnabled());

	// Update the distance and size for each bounds.
	StreamingData.UpdateBoundSizes_Async(Settings);
	
	ApplyPakStateChanges_Async();

	{
		TRACE_COUNTER_SET(StreamableAssets, StreamingRenderAssets.Num());		

		uint64 StartTime = FPlatformTime::Cycles64();

		for (FStreamingRenderAsset& StreamingRenderAsset : StreamingRenderAssets)
		{
			if (IsAborted()) break;

			StreamingRenderAsset.UpdateOptionalMipsState_Async();

			StreamingData.UpdatePerfectWantedMips_Async(StreamingRenderAsset, Settings);
			StreamingRenderAsset.DynamicBoostFactor = 1.f; // Reset after every computation.
		}

		uint64 LengthCycles64 = (FPlatformTime::Cycles64() - StartTime);
		double ElapsedMSTime = FPlatformTime::ToMilliseconds64(LengthCycles64);

		// use of a custom stat to have the elapsed time as a global stat and not a stat split accross multiple threads
		CSV_CUSTOM_STAT(TextureStreaming, RenderAssetStreamingUpdate, float(ElapsedMSTime), ECsvCustomStatOp::Set);
	}

	// According to budget, make relevant sacrifices and keep possible unwanted mips
	UpdateBudgetedMips_Async();
	UpdateBudgetedVirtualTextureMips_Async();

	// Update load requests.
	UpdateLoadAndCancelationRequests_Async();

	// Update bHasStreamingUpdatePending
	UpdatePendingStreamingStatus_Async();

	StreamingData.OnTaskDone_Async();

#if STATS
	UpdateStats_Async();
#elif UE_WITH_RENDER_ASSET_STREAMING_CSV_STATS
	UpdateCSVOnlyStats_Async();
#endif // STATS
}

void FRenderAssetStreamingMipCalcTask::UpdateStats_Async()
{
#if STATS
	FRenderAssetStreamingStats& Stats = StreamingManager.GatheredStats;
	FRenderAssetStreamingSettings& Settings = StreamingManager.Settings;
	TArray<FStreamingRenderAsset>& StreamingRenderAssets = StreamingManager.AsyncUnsafeStreamingRenderAssets;

	Stats.RenderAssetPool = PoolSize;
	// Stats.StreamingPool = MemoryBudget;
	Stats.UsedStreamingPool = 0;

	Stats.SafetyPool = MemoryMargin; 
	Stats.TemporaryPool = TempMemoryBudget;

#if ALLOW_OTHER_PLATFORM_CONFIG && WITH_EDITOR
	UDeviceProfileManager& DeviceProfileManager = UDeviceProfileManager::Get();
	UDeviceProfile* PreviewDeviceProfile = DeviceProfileManager.GetPreviewDeviceProfile();
	if (PreviewDeviceProfile)
	{
		const FString* PoolSizeString = PreviewDeviceProfile->GetAllExpandedCVars().Find(TEXT("r.Streaming.PoolSize"));
		if (PoolSizeString != nullptr)
		{
			int64 PoolSizePreview = FCString::Atoi(**PoolSizeString);
			MemoryBudget = PoolSizePreview * 1024 * 1024;
		}
	}
#endif
	Stats.StreamingPool = MemoryBudget;

	Stats.NonStreamingMips = AllocatedMemory;

	Stats.RequiredPool = 0;
#if WITH_EDITOR
	Stats.RequiredPool_Editor = 0;
#endif
	Stats.VisibleMips = 0;
	Stats.HiddenMips = 0;

	Stats.ForcedMips = 0;
	Stats.UnkownRefMips = 0;

	Stats.CachedMips = 0;

	Stats.WantedMips = 0;
	Stats.PendingRequests = 0;

	Stats.OverBudget = 0;

	Stats.NumStreamedMeshes = 0;
	Stats.AvgNumStreamedLODs = 0.f;
	Stats.AvgNumResidentLODs = 0.f;
	Stats.AvgNumEvictedLODs = 0.f;
	Stats.StreamedMeshMem = 0;
	Stats.ResidentMeshMem = 0;
	Stats.EvictedMeshMem = 0;
	int32 TotalNumStreamedLODs = 0;
	int32 TotalNumResidentLODs = 0;
	int32 TotalNumEvictedLODs = 0;

	Stats.VirtualTexturePool = Settings.VirtualTexturePoolSize * 1024 * 1024;
	Stats.NumVirtualTextures = 0;
	Stats.RequiredVirtualTexturePool = 0;
	Stats.UsedVirtualTexturePool = 0;

	for (const FStreamingRenderAsset& StreamingRenderAsset : StreamingRenderAssets)
	{
		if (IsAborted()) break;
		if (!StreamingRenderAsset.RenderAsset) continue;

		const int64 ResidentSize = StreamingRenderAsset.GetSize(StreamingRenderAsset.ResidentMips);
		const int64 RequiredSize = StreamingRenderAsset.GetSize(StreamingRenderAsset.GetPerfectWantedMips());
		const int64 BudgetedSize = StreamingRenderAsset.GetSize(StreamingRenderAsset.BudgetedMips);
		const int64 MaxSize = StreamingRenderAsset.GetSize(StreamingRenderAsset.MaxAllowedMips);
		const int64 VisibleWantedSize = StreamingRenderAsset.GetSize(StreamingRenderAsset.VisibleWantedMips);

		if (StreamingRenderAsset.IsVirtualTexture())
		{
			++Stats.NumVirtualTextures;
			Stats.RequiredVirtualTexturePool += RequiredSize;
			Stats.UsedVirtualTexturePool += ResidentSize;
			// For virtual textures we don't include the root pages in the budget since they are managed completely 
			// by the virtual texture system. We can't even assume that root pages are always locked because the virtual 
			// texture system usually only locks them when a material using the texture allocates a VT.
			const int64 RootPageSize = StreamingRenderAsset.GetSize(StreamingRenderAsset.MinAllowedMips);
			Stats.RequiredVirtualTexturePool -= RootPageSize;
			Stats.UsedVirtualTexturePool -= RootPageSize;
			continue;
		}

#if WITH_EDITOR
		if (StreamingRenderAsset.IsTexture() && StreamingRenderAsset.RenderAsset->bMarkAsEditorStreamingPool)
		{
			// Remove from the non streaming budget what is actually taken by streaming.
			Stats.NonStreamingMips -= RequiredSize;

			// Editor resources should be forced resident
			Stats.RequiredPool_Editor += RequiredSize;
			continue;
		}
#endif

		// How much the streamer would use if there was no limit.
		Stats.RequiredPool += RequiredSize;

		// How much the streamer actually use.
		Stats.UsedStreamingPool += FMath::Min<int64>(RequiredSize, BudgetedSize);

		// Remove from the non streaming budget what is actually taken by streaming.
		Stats.NonStreamingMips -= ResidentSize * StreamingRenderAsset.IsTexture();

		// All persistent mip bias bigger than the expected is considered overbudget.
		const int32 OverBudgetBias = FMath::Max<int32>(0, StreamingRenderAsset.BudgetMipBias - Settings.GlobalMipBias);
		Stats.OverBudget += StreamingRenderAsset.GetSize(StreamingRenderAsset.MaxAllowedMips + OverBudgetBias) - MaxSize;

		const int64 UsedSize = FMath::Min3<int64>(RequiredSize, BudgetedSize, ResidentSize);

		Stats.WantedMips += UsedSize;
		Stats.CachedMips += FMath::Max<int64>(ResidentSize - UsedSize, 0);

		if (GIsEditor && StreamingRenderAsset.GetFlag(FStreamingRenderAssetFlags::ForceFullyLoadHeuristic))
		{
			Stats.ForcedMips += UsedSize;
		}
		else if (StreamingRenderAsset.GetFlag(FStreamingRenderAssetFlags::UseUnkownRefHeuristic))
		{
			Stats.UnkownRefMips += UsedSize;
		}
		else
		{
			if (VisibleWantedSize >= UsedSize)
			{
				Stats.VisibleMips += UsedSize;
			}
			else // VisibleWantedSize < UsedSize
			{
				Stats.VisibleMips += VisibleWantedSize;

				// Forced mips are not the same as hidden mips as they are loaded because the user wants them absolutly
				if (StreamingRenderAsset.GetFlag(FStreamingRenderAssetFlags::ForceFullyLoadHeuristic)
					|| (StreamingRenderAsset.IsTexture()
						&& StreamingRenderAsset.LODGroup == TEXTUREGROUP_HierarchicalLOD
						&& Settings.HLODStrategy > 0))
				{
					Stats.ForcedMips += UsedSize - VisibleWantedSize;
				}
				else
				{
					Stats.HiddenMips += UsedSize - VisibleWantedSize;
				}
			}
		}

		if (StreamingRenderAsset.RequestedMips > StreamingRenderAsset.ResidentMips)
		{
			Stats.PendingRequests += StreamingRenderAsset.GetSize(StreamingRenderAsset.RequestedMips) - ResidentSize;
		}

		if (StreamingRenderAsset.IsMesh())
		{
			const int32 NumStreamedLODs = StreamingRenderAsset.MaxAllowedMips - StreamingRenderAsset.MinAllowedMips;
			const int32 NumResidentLODs = StreamingRenderAsset.ResidentMips;
			const int32 NumEvictedLODs = StreamingRenderAsset.MaxAllowedMips - NumResidentLODs;
			const int64 TotalSize = StreamingRenderAsset.GetSize(StreamingRenderAsset.MaxAllowedMips);
			const int64 StreamedSize = TotalSize - StreamingRenderAsset.GetSize(StreamingRenderAsset.MinAllowedMips);
			const int64 EvictedSize = TotalSize - ResidentSize;

			++Stats.NumStreamedMeshes;
			TotalNumStreamedLODs += NumStreamedLODs;
			TotalNumResidentLODs += NumResidentLODs;
			TotalNumEvictedLODs += NumEvictedLODs;
			Stats.StreamedMeshMem += StreamedSize;
			Stats.ResidentMeshMem += ResidentSize;
			Stats.EvictedMeshMem += EvictedSize;
		}
	}

	if (Stats.NumStreamedMeshes > 0)
	{
		Stats.AvgNumStreamedLODs = (float)TotalNumStreamedLODs / Stats.NumStreamedMeshes;
		Stats.AvgNumResidentLODs = (float)TotalNumResidentLODs / Stats.NumStreamedMeshes;
		Stats.AvgNumEvictedLODs = (float)TotalNumEvictedLODs / Stats.NumStreamedMeshes;
	}

	Stats.OverBudget += FMath::Max<int64>(Stats.RequiredPool - Stats.StreamingPool, 0);
	Stats.Timestamp = FPlatformTime::Seconds();
#endif
}

void FRenderAssetStreamingMipCalcTask::UpdateCSVOnlyStats_Async()
{
	const TArray<FStreamingRenderAsset>& StreamingRenderAssets = StreamingManager.AsyncUnsafeStreamingRenderAssets;

	FRenderAssetStreamingStats& Stats = StreamingManager.GatheredStats;

	Stats.RenderAssetPool = PoolSize;

	Stats.SafetyPool = MemoryMargin;
	Stats.TemporaryPool = TempMemoryBudget;
	Stats.StreamingPool = MemoryBudget;
	Stats.NonStreamingMips = AllocatedMemory;

	Stats.RequiredPool = 0;
	Stats.CachedMips = 0;
	Stats.WantedMips = 0;
	Stats.PendingRequests = 0;

	Stats.NumStreamedMeshes = 0;
	Stats.AvgNumStreamedLODs = 0.f;
	Stats.AvgNumResidentLODs = 0.f;
	Stats.AvgNumEvictedLODs = 0.f;
	Stats.StreamedMeshMem = 0;
	Stats.ResidentMeshMem = 0;
	Stats.EvictedMeshMem = 0;
	int32 TotalNumStreamedLODs = 0;
	int32 TotalNumResidentLODs = 0;
	int32 TotalNumEvictedLODs = 0;

	for (const FStreamingRenderAsset& StreamingRenderAsset : StreamingRenderAssets)
	{
		if (IsAborted()) break;
		if (!StreamingRenderAsset.RenderAsset) continue;
		if (StreamingRenderAsset.IsVirtualTexture()) continue;

		const int64 ResidentSize = StreamingRenderAsset.GetSize(StreamingRenderAsset.ResidentMips);
		const int64 RequiredSize = StreamingRenderAsset.GetSize(StreamingRenderAsset.GetPerfectWantedMips());
		const int64 BudgetedSize = StreamingRenderAsset.GetSize(StreamingRenderAsset.BudgetedMips);

		// How much the streamer would use if there was no limit.
		Stats.RequiredPool += RequiredSize;

		// Remove from the non streaming budget what is actually taken by streaming.
		Stats.NonStreamingMips -= ResidentSize * StreamingRenderAsset.IsTexture();

		const int64 UsedSize = FMath::Min3<int64>(RequiredSize, BudgetedSize, ResidentSize);

		Stats.WantedMips += UsedSize;
		Stats.CachedMips += FMath::Max<int64>(ResidentSize - UsedSize, 0);

		if (StreamingRenderAsset.RequestedMips > StreamingRenderAsset.ResidentMips)
		{
			Stats.PendingRequests += StreamingRenderAsset.GetSize(StreamingRenderAsset.RequestedMips) - ResidentSize;
		}

		if (StreamingRenderAsset.IsMesh())
		{
			const int32 NumStreamedLODs = StreamingRenderAsset.MaxAllowedMips - StreamingRenderAsset.MinAllowedMips;
			const int32 NumResidentLODs = StreamingRenderAsset.ResidentMips;
			const int32 NumEvictedLODs = StreamingRenderAsset.MaxAllowedMips - NumResidentLODs;
			const int64 TotalSize = StreamingRenderAsset.GetSize(StreamingRenderAsset.MaxAllowedMips);
			const int64 StreamedSize = TotalSize - StreamingRenderAsset.GetSize(StreamingRenderAsset.MinAllowedMips);
			const int64 EvictedSize = TotalSize - ResidentSize;

			++Stats.NumStreamedMeshes;
			TotalNumStreamedLODs += NumStreamedLODs;
			TotalNumResidentLODs += NumResidentLODs;
			TotalNumEvictedLODs += NumEvictedLODs;
			Stats.StreamedMeshMem += StreamedSize;
			Stats.ResidentMeshMem += ResidentSize;
			Stats.EvictedMeshMem += EvictedSize;
		}
	}

	if (Stats.NumStreamedMeshes > 0)
	{
		Stats.AvgNumStreamedLODs = (float)TotalNumStreamedLODs / Stats.NumStreamedMeshes;
		Stats.AvgNumResidentLODs = (float)TotalNumResidentLODs / Stats.NumStreamedMeshes;
		Stats.AvgNumEvictedLODs = (float)TotalNumEvictedLODs / Stats.NumStreamedMeshes;
	}
}
