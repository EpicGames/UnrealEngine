// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
AsyncTextureStreaming.h: Definitions of classes used for texture streaming async task.
=============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "Stats/Stats.h"
#include "ContentStreaming.h"
#include "Async/AsyncWork.h"
#include "Streaming/StreamingTexture.h"
#include "Streaming/TextureInstanceView.h"

class FLevelRenderAssetManager;
class FDynamicRenderAssetInstanceManager;
struct FRenderAssetStreamingManager;

/** Thread-safe helper struct for streaming information. */
class FAsyncRenderAssetStreamingData
{
public:

	/** Set the data but do as little as possible, called from the game thread. */
	void Init(
		TArray<FStreamingViewInfo> InViewInfos,
		float InWorldTime,
		TArray<FLevelRenderAssetManager*>& LevelStaticInstanceManagers,
		FDynamicRenderAssetInstanceManager& DynamicComponentManager);

	void Init(
		TArray<FStreamingViewInfo> InViewInfos,
		float InLastUpdateTime);

	void ComputeViewInfoExtras(const FRenderAssetStreamingSettings& Settings);

	/** Update everything internally so to allow calls to CalcWantedMips */
	void UpdateBoundSizes_Async(const FRenderAssetStreamingSettings& Settings);

	void UpdatePerfectWantedMips_Async(FStreamingRenderAsset& StreamingRenderAsset, const FRenderAssetStreamingSettings& Settings, bool bOutputToLog = false) const;

	uint32 GetAllocatedSize() const { return ViewInfos.GetAllocatedSize() + StaticInstancesViews.GetAllocatedSize(); }

	FORCEINLINE const FRenderAssetInstanceAsyncView& GetDynamicInstancesView() const { return DynamicInstancesView; }
	FORCEINLINE const TArray<FRenderAssetInstanceAsyncView>& GetStaticInstancesViews() const { return StaticInstancesViews; }
	FORCEINLINE const TArray<FStreamingViewInfo>& GetViewInfos() const { return ViewInfos; }

	FORCEINLINE bool HasAnyView() const { return ViewInfos.Num() > 0; }

	// Release the view. Decrementing the refcounts. 
	// This must be done on the gamethread as the refcount are not threadsafe.
	void ReleaseViews()
	{
		DynamicInstancesView = FRenderAssetInstanceAsyncView();
		StaticInstancesViews.Reset();
	}

	void OnTaskDone_Async() 
	{ 
		DynamicInstancesView.OnTaskDone();
		for (FRenderAssetInstanceAsyncView& StaticView : StaticInstancesViews)
		{
			StaticView.OnTaskDone();
		}
	}


private:
	/** Cached from FStreamingManagerBase. */
	TArray<FStreamingViewInfo> ViewInfos;
	
	/** View related data taking view boost into account */
	FStreamingViewInfoExtraArray ViewInfoExtras;

	FRenderAssetInstanceAsyncView DynamicInstancesView;

	/** Cached from each ULevel. */
	TArray<FRenderAssetInstanceAsyncView> StaticInstancesViews;

	/** Time since last full update. Used to know if something is immediately visible. */
	float LastUpdateTime;

	float MaxScreenSizeOverAllViews;

	/** Sorted list of all static instances. The sorting is based on MaxLevelTextureScreenSize. */
	TArray<int32> StaticInstancesViewIndices;

	/** List of all static instances that were rejected. */
	TArray<int32> CulledStaticInstancesViewIndices;

	/** list of all static instances based on level index */
	TArray<int32> StaticInstancesViewLevelIndices;
};

struct FCompareRenderAssetByRetentionPriority // Bigger retention priority first.
{
	FCompareRenderAssetByRetentionPriority(const TArray<FStreamingRenderAsset>& InStreamingRenderAssets) : StreamingRenderAssets(InStreamingRenderAssets) {}
	const TArray<FStreamingRenderAsset>& StreamingRenderAssets;

	FORCEINLINE bool operator()( int32 IndexA, int32 IndexB ) const
	{
		const int32 PrioA = StreamingRenderAssets[IndexA].RetentionPriority;
		const int32 PrioB = StreamingRenderAssets[IndexB].RetentionPriority;
		if (PrioA > PrioB)  return true;
		if (PrioA == PrioB)
		{
			const float SSA = StreamingRenderAssets[IndexA].NormalizedScreenSize;
			const float SSB = StreamingRenderAssets[IndexB].NormalizedScreenSize;
			if (SSA > SSB) return true;
			if (SSA == SSB) return IndexA > IndexB;  // Sorting by index so that it gets deterministic.
		}
		return false;
	}
};

struct FCompareRenderAssetByLoadOrderPriority // Bigger load order priority first.
{
	FCompareRenderAssetByLoadOrderPriority(const TArray<FStreamingRenderAsset>& InStreamingRenderAssets) : StreamingRenderAssets(InStreamingRenderAssets) {}
	const TArray<FStreamingRenderAsset>& StreamingRenderAssets;

	FORCEINLINE bool operator()( int32 IndexA, int32 IndexB ) const
	{
		const int32 PrioA = StreamingRenderAssets[IndexA].LoadOrderPriority;
		const int32 PrioB = StreamingRenderAssets[IndexB].LoadOrderPriority;
		if ( PrioA > PrioB )  return true;
		if ( PrioA == PrioB ) return IndexA > IndexB;  // Sorting by index so that it gets deterministic.
		return false;
	}
};

/** Async work for calculating priorities and target number of mips for all textures/meshes. */
// this could implement a better abandon, but give how it is used, it does that anyway via the abort mechanism
class FRenderAssetStreamingMipCalcTask : public FNonAbandonableTask
{
public:

	FRenderAssetStreamingMipCalcTask( FRenderAssetStreamingManager* InStreamingManager )
	:	StreamingManager( *InStreamingManager )
	,	bAbort( false )
	{
		Reset(0, 0, 0, 0, 0);
		MemoryBudget = 0;
		MeshMemoryBudget = 0;
		PerfectWantedMipsBudgetResetThresold = 0;
	}

	/** Resets the state to start a new async job. */
	void Reset(int64 InTotalGraphicsMemory, int64 InAllocatedMemory, int64 InPoolSize, int64 InTempMemoryBudget, int64 InMemoryMargin)
	{
		TotalGraphicsMemory = InTotalGraphicsMemory;
		AllocatedMemory = InAllocatedMemory;
		PoolSize = InPoolSize;
		TempMemoryBudget = InTempMemoryBudget;
		MemoryMargin = InMemoryMargin;

		bAbort = false;
	}

	/** Notifies the async work that it should abort the thread ASAP. */
	void Abort() { bAbort = true; }

	/** Whether the async work is being aborted. Can be used in conjunction with IsDone() to see if it has finished. */
	bool IsAborted() const { return bAbort;	}

	/** Returns the resulting priorities, matching the FRenderAssetStreamingManager::StreamingRenderAssets array. */
	const TArray<int32>& GetLoadRequests() const { return LoadRequests; }
	const TArray<int32>& GetCancelationRequests() const { return CancelationRequests; }
	const TArray<int32>& GetPendingUpdateDirties() const { return PendingUpdateDirties; }

	FAsyncRenderAssetStreamingData StreamingData;
	
	/** Performs the async work. */
	void DoWork();

	bool HasAnyView() const { return StreamingData.HasAnyView(); }

	void ReleaseAsyncViews()
	{
		StreamingData.ReleaseViews();
	}

protected:

	/** Ensures that no temporary streaming boost are active which could interfere with render asset streaming bias in undesirable ways. */
	bool AllowPerRenderAssetMipBiasChanges() const;

private:

	friend class FAsyncTask<FRenderAssetStreamingMipCalcTask>;

	void ApplyPakStateChanges_Async();

	void TryDropMaxResolutions(TArray<int32>& PrioritizedRenderAssets, int64& MemoryBudgeted, const int64 InMemoryBudget);

	void TryDropMips(TArray<int32>& PrioritizedRenderAssets, int64& MemoryBudgeted, const int64 InMemoryBudget);

	void TryKeepMips(TArray<int32>& PrioritizedRenderAssets, int64& MemoryBudgeted, const int64 InMemoryBudget);

	void UpdateBudgetedMips_Async();
	
	void UpdateBudgetedVirtualTextureMips_Async();

	void UpdateLoadAndCancelationRequests_Async();

	void UpdatePendingStreamingStatus_Async();

	void UpdateStats_Async();

	void UpdateCSVOnlyStats_Async();

	FORCEINLINE TStatId GetStatId() const
	{
		RETURN_QUICK_DECLARE_CYCLE_STAT(FRenderAssetStreamingMipCalcTask, STATGROUP_ThreadPoolAsyncTasks);
	}

	/** Reference to the owning streaming manager, for accessing the thread-safe data. */
	FRenderAssetStreamingManager& StreamingManager;

	/** Indices for load/unload requests, sorted by load order. */
	TArray<int32>	LoadRequests;
	TArray<int32>	CancelationRequests;

	/** Indices of texture with dirty values for bHasUpdatePending */
	TArray<int32>	PendingUpdateDirties;

	/** Whether the async work should abort its processing. */
	volatile bool				bAbort;

	/** How much VRAM the hardware has. */
	int64 TotalGraphicsMemory;

	/** How much gpu resources are currently allocated in the texture/mesh pool (all category). */
	int64 AllocatedMemory; 

	/** Size of the pool once non streaming data is removed and value is stabilized */
	int64 PoolSize;

	/** How much temp memory is allowed (temp memory is taken when changing mip count). */
	int64 TempMemoryBudget;

	/** How much temp memory is allowed (temp memory is taken when changing mip count). */
	int64 MemoryMargin;

	/** How much memory is available for textures/meshes. */
	int64 MemoryBudget;

	/** How much memory is available for meshes if a separate pool is used. */
	int64 MeshMemoryBudget;

	/**
	 * The value of all required mips (without memory constraint) used to trigger a budget reset. 
	 * Whenever the perfect wanted mips drops significantly, we reset the budget to avoid keeping 
	 * resolution constraint used to fit that previous situation.
	 */
	int64 PerfectWantedMipsBudgetResetThresold;
};
