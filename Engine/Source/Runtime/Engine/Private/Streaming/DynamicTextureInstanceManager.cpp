// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	DynamicTextureInstanceManager.cpp: Implementation of content streaming classes.
=============================================================================*/

#include "Streaming/DynamicTextureInstanceManager.h"
#include "Streaming/StreamingManagerTexture.h"
#include "Components/PrimitiveComponent.h"
#include "Async/ParallelFor.h"
#include "Materials/MaterialInterface.h"

// TODO: Remove this cvar and old code path once effectiveness and correctness is verified
static int32 GStreamingDeferredRemoveDyanmicInstances = 1;
static FAutoConsoleVariableRef CVarStreamingDeferredRemoveDynamicInstances(
	TEXT("r.Streaming.DeferredRemoveDynamicInstances"),
	GStreamingDeferredRemoveDyanmicInstances,
	TEXT("Whether to defer removing components from the dynamic instance manager to avoid stalling the game thread on component deregistration."),
	ECVF_Default);

void FDynamicRenderAssetInstanceManager::FTasks::SyncResults()
{
	// Update the bounds first as we want the async view to be fully up-to-date.
	RefreshFullTask->TryWork(false);
	CreateViewTask->TryWork(false);
	// All (async) work must be completed before synching the results as the work assume a constant state.
	RefreshFullTask->TrySync();
	CreateViewTask->TrySync();
}

void FDynamicRenderAssetInstanceManager::FTasks::SyncRefreshFullTask()
{
	RefreshFullTask->TryWork(false);
	RefreshFullTask->TrySync();
}

FDynamicRenderAssetInstanceManager::FDynamicRenderAssetInstanceManager(FOnSyncDoneDelegate&& InOnSyncDoneDelegate)
	: StateSync(FRenderAssetDynamicInstanceStateTaskSync<FTasks>::FOnSyncDone::CreateLambda(MoveTemp(InOnSyncDoneDelegate)))
	, DirtyIndex(0)
	, PendingDefragSrcBoundIndex(INDEX_NONE)
	, PendingDefragDstBoundIndex(INDEX_NONE)
{
	FTasks& Tasks = StateSync.GetTasks();
	Tasks.RefreshFullTask = new FRefreshFullTask(RenderAssetInstanceTask::FRefreshFull::FOnWorkDone::CreateLambda([this](int32 InBeginIndex, int32 InEndIndex, const TArray<int32>& SkippedIndices, int32 FirstFreeBound, int32 LastUsedBound){ this->OnRefreshVisibilityDone(InBeginIndex, InEndIndex, SkippedIndices, FirstFreeBound, LastUsedBound); }));
	Tasks.CreateViewTask = new FCreateViewTask(RenderAssetInstanceTask::FCreateViewWithUninitializedBounds::FOnWorkDone::CreateLambda([this](FRenderAssetInstanceView* InView){ this->OnCreateViewDone(InView); }));
}

FDynamicRenderAssetInstanceManager::~FDynamicRenderAssetInstanceManager()
{
	StateSync.Sync();
}

bool FDynamicRenderAssetInstanceManager::IsReferenced(const IPrimitiveComponent* Component) const
{ 
	return Component && Component->GetStreamingState().IsAttachedToStreamingManagerAsDynamic() && (StateSync.GetState()->HasComponentReferences(Component) || PendingComponents.Contains(Component));
}

void FDynamicRenderAssetInstanceManager::RegisterTasks(RenderAssetInstanceTask::FDoWorkTask& AsyncTask)
{
	FTasks& Tasks = StateSync.GetTasks();
	AsyncTask.Add(Tasks.RefreshFullTask.GetReference());
	AsyncTask.Add(Tasks.CreateViewTask.GetReference());
}

void FDynamicRenderAssetInstanceManager::IncrementalUpdate(FRemovedRenderAssetArray& RemovedRenderAssets, float Percentage)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FDynamicRenderAssetInstanceManager::IncrementalUpdate);
	FRenderAssetInstanceState* State = StateSync.SyncAndGetState();

	// First try to apply the pending defrag.
	if (PendingDefragSrcBoundIndex != INDEX_NONE && PendingDefragDstBoundIndex != INDEX_NONE)
	{
		if (State->MoveBound(PendingDefragSrcBoundIndex, PendingDefragDstBoundIndex))
		{
			State->TrimBounds(); //Release any trailing bounds.
		}
		PendingDefragSrcBoundIndex = INDEX_NONE;
		PendingDefragDstBoundIndex = INDEX_NONE;
	}
	
	// Because PendingComponents could have duplicates, we first do a pass to remove everything.
	for (const IPrimitiveComponent* Component : PendingComponents)
	{
		check(Component);

		State->RemoveComponent(Component, &RemovedRenderAssets);
		Component->GetStreamingState().SetAttachedToStreamingManagerAsDynamic(false);
		// Re-enable updates now that the component is out of the pending list.
		Component->GetStreamingState().SetIgnoreStreamingManagerUpdate(false);
	}

	// Now insert everything, checking for duplicates through bAttachedToStreamingManagerAsDynamic
	if (PendingComponents.Num())
	{
		const int32 ToProcessComponentCount = PendingComponents.Num();
		int32 MinBatchSize = 1;
		if (FRenderAssetStreamingManager::IsParallelForAllowedDuringIncrementalUpdate(ToProcessComponentCount, MinBatchSize))
		{
			TArray<FRenderAssetInstanceState::FPreAddComponentIgnoreBoundsPayload> Payloads;
			Payloads.SetNum(ToProcessComponentCount);
			ParallelFor(TEXT("PreAddComponentsIgnoreBounds"), ToProcessComponentCount, MinBatchSize, [&](int32 Index)
			{
				FTaskTagScope Scope(ETaskTag::EParallelGameThread);
				const IPrimitiveComponent* Component = PendingComponents[Index];
				check(Component);
				if (!Component->GetStreamingState().IsAttachedToStreamingManagerAsDynamic() && CanManage(Component) && Component->IsRegistered() && (!Component->IsRenderStateCreated() || Component->GetSceneProxy()))
				{
					FStreamingTextureLevelContext LevelContext(GetCurrentMaterialQualityLevelChecked(), Component);
					// Because LevelContext is not thread-safe when ComponentBuildData is valid, we force not to use built data
					LevelContext.SetForceNoUseBuiltData(true);
					FRenderAssetInstanceState::PreAddComponentIgnoreBounds(Component, LevelContext, Payloads[Index]);
				}
				else
				{
					Payloads[Index].Result = EAddComponentResult::Fail;
				}
			});
			{
				for (FRenderAssetInstanceState::FPreAddComponentIgnoreBoundsPayload& Payload : Payloads)
				{
					if (State->AddComponentIgnoreBounds(Payload) == EAddComponentResult::Success)
					{
						Payload.Component->GetStreamingState().SetAttachedToStreamingManagerAsDynamic(true);
					}
				}
			}
		}
		else
		{
			for (const IPrimitiveComponent* Component : PendingComponents)
			{
				check(Component);

				// Check bAttachedToStreamingManagerAsDynamic here to prevent handling the component twice.
				if (!Component->GetStreamingState().IsAttachedToStreamingManagerAsDynamic() && CanManage(Component) && Component->IsRegistered() && (!Component->IsRenderStateCreated() || Component->GetSceneProxy()))
				{
					FStreamingTextureLevelContext LevelContext(GetCurrentMaterialQualityLevelChecked(), Component);
					if (State->AddComponentIgnoreBounds(Component, LevelContext) == EAddComponentResult::Success)
					{
						Component->GetStreamingState().SetAttachedToStreamingManagerAsDynamic(true);
					}
				}
			}
		}
		PendingComponents.Reset();
	}

	// Now update the bounds
	Refresh(Percentage);
}

void FDynamicRenderAssetInstanceManager::OnCreateViewDone(FRenderAssetInstanceView* InView)
{
	// Don't call get state here to prevent recursion as this is a task callback.
	FRenderAssetInstanceState* State = StateSync.GetStateUnsafe();
	check(DirtyIndex >= State->NumBounds()); // Must be fully valid or the swap will be destructive.

	// The tasks creates dirty bounds, so after sync, move the valid bounds to the view and mark dirty all current bounds.
	// The incremental update will refresh the bounds within the next update loop.
	FRenderAssetInstanceView::SwapData(InView, State);

	// All of the data is dirty now since we just got uninitialized data.
	DirtyIndex = 0;

	AsyncView = TRefCountPtr<const FRenderAssetInstanceView>(InView);
}

void FDynamicRenderAssetInstanceManager::OnRefreshVisibilityDone(int32 BeginIndex, int32 EndIndex, const TArray<int32>& SkippedIndices, int32 FirstFreeBound, int32 LastUsedBound)
{
	// Don't call get state here to prevent recursion as this is a task callback.
	FRenderAssetInstanceState* State = StateSync.GetStateUnsafe();
	check(DirtyIndex == BeginIndex);

	for (int32 SkippedIndex : SkippedIndices)
	{
		State->UpdateBounds(SkippedIndex);
	}
	DirtyIndex = EndIndex;

	// Move the last valid bound to the first empty place, trying to free the tail.
	if (CVarStreamingDefragDynamicBounds.GetValueOnGameThread() > 0)
	{
		if (FirstFreeBound < LastUsedBound)
		{
			PendingDefragDstBoundIndex = FirstFreeBound;
			PendingDefragSrcBoundIndex = LastUsedBound;
		}
	}
}

void FDynamicRenderAssetInstanceManager::OnPreGarbageCollect(FRemovedRenderAssetArray& RemovedRenderAssets)
{
	for (int32 Index = 0; Index < PendingComponents.Num(); ++Index)
	{
		const IPrimitiveComponent* Primitive = PendingComponents[Index];
		check(Primitive);

		// If the component is not registered anymore, remove it. If it gets registered again it will be reinserted in the pending list.
		// This allows to remove all unregistered components at once without having to handle each of them in FDynamicRenderAssetInstanceManager::Remove().
		// The goal here is to bypass the possibly slow search in PendingComponents.
		if (!Primitive->IsRegistered() || !CanManage(Primitive))
		{
			PendingComponents.RemoveAtSwap(Index);
			Primitive->GetStreamingState().SetIgnoreStreamingManagerUpdate(false);
			--Index;

			if (StateSync.GetState()->HasComponentReferences(Primitive))
			{
				if (GStreamingDeferredRemoveDyanmicInstances != 0)
				{
					// Prevent the refresh full task from dereferencing possibly soon to be GCed components
					StateSync.GetTasks().SyncRefreshFullTask();
					// Only clear references to avoid a sync with the create async view task
					StateSync.GetStateUnsafe()->RemoveComponentReferences(Primitive);
				}
				else
				{
					StateSync.SyncAndGetState()->RemoveComponent(Primitive, &RemovedRenderAssets);
				}
			}
			Primitive->GetStreamingState().SetAttachedToStreamingManagerAsDynamic(false);
		}
	}
}

bool FDynamicRenderAssetInstanceManager::CanManage(const IPrimitiveComponent* Component) const
{
	if (!Component)
	{
		return false;
	}
	const UObject* ComponentObject = Component->GetUObject();
	return IsValid(ComponentObject) && !ComponentObject->HasAnyFlags(RF_BeginDestroyed|RF_FinishDestroyed);
}

void FDynamicRenderAssetInstanceManager::Refresh(float Percentage)
{
	static bool bUseBackgroupTask = true;

	QUICK_SCOPE_CYCLE_COUNTER(FDynamicRenderAssetInstanceManager_Refresh);

	// Even if the incremental update does not do any allocation, the tasks must be completed in order to udpate the new dirty arrays - see FTasks::SyncResults.
	FRenderAssetInstanceState* State = StateSync.SyncAndGetState();

	if (DirtyIndex < State->NumBounds())
	{
		const int32 EndIndex = FMath::Min(State->NumBounds(), DirtyIndex + FMath::CeilToInt((float)State->NumBounds() * Percentage));
		StateSync.GetTasks().RefreshFullTask->Init(State, DirtyIndex, EndIndex);
	}
}

EAddComponentResult FDynamicRenderAssetInstanceManager::Add(const IPrimitiveComponent* Component, FStreamingTextureLevelContext& LevelContext, float MaxAllowedUIDensity)
{
	// Don't cull out primitives with no SceneProxy because they need to be removed first (ex: if the primitive got hidden).
	if (CanManage(Component))
	{
		FRenderAssetOwnerStreamingState& StreamingState = Component->GetStreamingState();

		// This flag stays true forever to notify that this will always be handled as dynamic from now on.
		// To is to allow the update (on render state changes) to work, which handles only dynamic primitives
		StreamingState.SetHandledByStreamingManagerAsDynamic(true);

		// If the update is disabled, it's either already in the pending list or this component has no streaming data.
		if (!StreamingState.IsIgnoreStreamingManagerUpdate())
		{
			// Ignore any further updates while the component is in the pending list.
			StreamingState.SetIgnoreStreamingManagerUpdate(true);

			// Postpone so that we don't have to sync the state.
			PendingComponents.Add(Component);

			// Notify attached since it is now referred in PendingComponents
			StreamingState.SetAttachedToStreamingManagerAsDynamic(true);
		}

		return EAddComponentResult::Success;
	}

	return EAddComponentResult::Fail;
}

void FDynamicRenderAssetInstanceManager::Remove(const IPrimitiveComponent* Component, FRemovedRenderAssetArray* RemovedRenderAssets)
{
	check(!Component || !Component->GetUObject() || Component->GetUObject()->IsValidLowLevelFast());
	if (Component && Component->GetStreamingState().IsAttachedToStreamingManagerAsDynamic())
	{
		PendingComponents.RemoveSwap(Component);
		Component->GetStreamingState().SetIgnoreStreamingManagerUpdate(false);

		// If the component is used, stop any task possibly indirecting it, and clear references.
		if (StateSync.GetState()->HasComponentReferences(Component))
		{
			if (GStreamingDeferredRemoveDyanmicInstances != 0)
			{
				// Only clear references to avoid syncing with refresh full and create async view tasks
				StateSync.GetStateUnsafe()->RemoveComponentReferences(Component);
			}
			else
			{
				StateSync.SyncAndGetState()->RemoveComponent(Component, RemovedRenderAssets);
			}
		}
		Component->GetStreamingState().SetAttachedToStreamingManagerAsDynamic(false);
	}
}

void FDynamicRenderAssetInstanceManager::PrepareAsyncView()
{
	// Terminate any pending work as we are about to create a new task.
	const FRenderAssetInstanceState* State = StateSync.SyncAndGetState();

	// Update any dirty bounds as everything must be up-do-date before making the view.
	// Bounds can be dirty even after calling FDynamicRenderAssetInstanceManager::IncrementalUpdate as newly added bounds do not update DirtyIndex.
	Refresh(1.f);

	 // Can only release the old view if no one else is referring it, as the refcount is not threadsafe.
	StateSync.GetTasks().CreateViewTask->Init(State, AsyncView.GetRefCount() == 1 ? AsyncView.GetReference() : nullptr);

	// Safe release must be called before the StartBackgroundTask() to be threadsafe.
	AsyncView.SafeRelease();
}

const FRenderAssetInstanceView* FDynamicRenderAssetInstanceManager::GetAsyncView(bool bCreateIfNull)
{
	FRenderAssetInstanceState* State = StateSync.SyncAndGetState();
	if (!AsyncView && bCreateIfNull)
	{
		AsyncView = FRenderAssetInstanceView::CreateView(State);
	}
	DirtyIndex = 0; // Force a full refresh!
	return AsyncView.GetReference();
}

uint32 FDynamicRenderAssetInstanceManager::GetAllocatedSize() const
{ 
	const FRenderAssetInstanceState* State = StateSync.GetState();
	return  State ? (sizeof(FRenderAssetInstanceState) + State->GetAllocatedSize()) : 0;
}

const FRenderAssetInstanceView* FDynamicRenderAssetInstanceManager::GetGameThreadView()
{
	check(IsInGameThread());
	return StateSync.GetState();
}
