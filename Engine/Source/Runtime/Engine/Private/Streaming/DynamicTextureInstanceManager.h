// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Streaming/TextureInstanceTask.h"

/** 
 * A texture/mesh instance manager to manage dynamic components. 
 * The async view generated is duplicated so that the state can change freely.
 */
class FDynamicRenderAssetInstanceManager
{
public:

	using FOnSyncDoneDelegate = TFunction<void (const FRemovedRenderAssetArray&)>;

	/** Constructor. */
	FDynamicRenderAssetInstanceManager(FOnSyncDoneDelegate&& InOnSyncDoneDelegate);
	~FDynamicRenderAssetInstanceManager();

	void RegisterTasks(RenderAssetInstanceTask::FDoWorkTask& AsyncTask);

	void IncrementalUpdate(FRemovedRenderAssetArray& RemovedRenderAssets, float Percentage);

	// Iterate all (non removed) components referred by the manager. Debug only.
	template<typename CallableT>
	void ForEachReferencedComponent(CallableT&& Callable)
	{
		StateSync.SyncAndGetState()->ForEachReferencedComponent(Forward<CallableT>(Callable));
	}

	/** Remove all pending components that are marked for delete. This prevents searching in the pending list for each entry. */
	void OnPreGarbageCollect(FRemovedRenderAssetArray& RemovedRenderAssets);

	/** Return whether this component can be managed by this manager. */
	bool IsReferenced(const IPrimitiveComponent* Component) const;

	/** Return whether this component is be managed by this manager. */
	bool CanManage(const IPrimitiveComponent* Component) const;

	/** Add a component streaming data, the LevelContext gives support for precompiled data. */
	EAddComponentResult Add(const IPrimitiveComponent* Component, FStreamingTextureLevelContext& LevelContext, float MaxAllowedUIDensity = 0);

	/** Remove a component, the RemovedRenderAssets is the list of textures or meshes not referred anymore. */
	 void Remove(const IPrimitiveComponent* Component, FRemovedRenderAssetArray* RemovedRenderAssets);

	/** Notify the manager that an async view will be requested on the next frame. */
	void PrepareAsyncView();

	/** Return a view of the data that has to be 100% thread safe. The content is allowed to be updated, but not memory must be reallocated. */
	const FRenderAssetInstanceView* GetAsyncView(bool bCreateIfNull);

	/** Return the size taken for sub-allocation. */
	uint32 GetAllocatedSize() const;

	const FRenderAssetInstanceView* GetGameThreadView();

protected:
	
	/** Refresh component data (bounds, last render time, min and max view distance) - see TextureInstanceView. */
	void Refresh(float Percentage);

	void OnCreateViewDone(FRenderAssetInstanceView* InView);
	void OnRefreshVisibilityDone(int32 BeginIndex, int32 EndIndex, const TArray<int32>& SkippedIndices, int32 FirstFreeBound, int32 LastUsedBound);

private:

	typedef RenderAssetInstanceTask::FCreateViewWithUninitializedBoundsTask FCreateViewTask;
	typedef RenderAssetInstanceTask::FRefreshFullTask FRefreshFullTask;

	struct FTasks
	{
		~FTasks()
		{
			SyncResults();
		}
		
		void SyncResults();
		
		void SyncRefreshFullTask();

		TRefCountPtr<FCreateViewTask> CreateViewTask;
		TRefCountPtr<FRefreshFullTask> RefreshFullTask;
	};

	/** The texture/mesh instances. Shared with the async task. */
	FRenderAssetDynamicInstanceStateTaskSync<FTasks> StateSync;

	/** A duplicate view for the async streaming task. */
	TRefCountPtr<const FRenderAssetInstanceView> AsyncView;

	/** Ranges from 0 to Bounds4Components.Num(). Used in the incremental update to update bounds and visibility. */
	int32 DirtyIndex;

	/** The valid bound index to be moved for defrag. */
	int32 PendingDefragSrcBoundIndex;
	/** The free bound index to be used as defrag destination. */
	int32 PendingDefragDstBoundIndex;

	/** The list of components to be processed (could have duplicates). */
	TArray<const IPrimitiveComponent*> PendingComponents;
};
