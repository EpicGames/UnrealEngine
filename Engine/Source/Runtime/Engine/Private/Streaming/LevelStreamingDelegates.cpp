// Copyright Epic Games, Inc. All Rights Reserved.

#include "Streaming/LevelStreamingDelegates.h"

TMulticastDelegate<void(
	UWorld*, 
	const ULevelStreaming*, 
	ULevel* LevelIfLoaded,
	ELevelStreamingState Current,
	ELevelStreamingTargetState PrevTarget, 
	ELevelStreamingTargetState NewTarget)> FLevelStreamingDelegates::OnLevelStreamingTargetStateChanged;

TMulticastDelegate<void(
	UWorld*, 
	const ULevelStreaming*,
	ULevel* LevelIfLoaded,
	ELevelStreamingState PreviousState,
	ELevelStreamingState NewState
	)> FLevelStreamingDelegates::OnLevelStreamingStateChanged;

TMulticastDelegate<void(UWorld*, const ULevelStreaming*, ULevel* LoadedLevel)> FLevelStreamingDelegates::OnLevelBeginMakingVisible;
TMulticastDelegate<void(UWorld*, const ULevelStreaming*, ULevel* LoadedLevel)> FLevelStreamingDelegates::OnLevelBeginMakingInvisible;

