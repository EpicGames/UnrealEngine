// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	LevelTextureManager.cpp: Implementation of content streaming classes.
=============================================================================*/

#include "Streaming/LevelTextureManager.h"
#include "Components/PrimitiveComponent.h"
#include "Engine/Level.h"
#include "Engine/World.h"
#include "SceneInterface.h"
#include "Streaming/DynamicTextureInstanceManager.h"
#include "Streaming/StaticTextureInstanceManager.h"
#include "Streaming/StreamingManagerTexture.h"
#include "Async/ParallelFor.h"
#include "Materials/MaterialInterface.h"
#include "Algo/ForEach.h"

FLevelRenderAssetManager::FLevelRenderAssetManager(ULevel* InLevel, RenderAssetInstanceTask::FDoWorkTask& AsyncTask)
	: Level(InLevel)
	, bIsInitialized(false)
	, bHasBeenReferencedToStreamedTextures(false)
	, StaticInstances(AsyncTask)
	, BuildStep(EStaticBuildStep::BuildTextureLookUpMap)
{
	if (Level)
	{
		Level->bStaticComponentsRegisteredInStreamingManager = false;
	}
}


void FLevelRenderAssetManager::Remove(FRemovedRenderAssetArray* RemovedRenderAssets)
{ 
	auto DetachComponent = [](const IPrimitiveComponent* PrimitiveComponent)
	{
		if (PrimitiveComponent)
		{
			// Check that this component was not already destroyed.
			check(!PrimitiveComponent->GetUObject() || PrimitiveComponent->GetUObject()->IsValidLowLevelFast());

			// Don't check as there can be duplicates in PendingComponents
			//check(PrimitiveComponent->IsAttachedToStreamingManagerAsStatic());

			// A component can only be referenced in one level, so if it was here, we can clear the flag
			PrimitiveComponent->GetStreamingState().SetAttachedToStreamingManagerAsStatic(false);
		}
	};

	StaticInstances.ForEachReferencedComponent(DetachComponent);
	Algo::ForEach(UnprocessedComponents, DetachComponent);
	Algo::ForEach(PendingComponents, DetachComponent);

	// Mark all static textures/meshes for removal.
	if (RemovedRenderAssets)
	{
		for (auto It = StaticInstances.GetRenderAssetIterator(); It; ++It)
		{
			RemovedRenderAssets->Push(*It);
		}
	}

	BuildStep = EStaticBuildStep::BuildTextureLookUpMap;
	UnprocessedComponents.Empty();
	PendingComponents.Empty();
	TextureGuidToLevelIndex.Empty();
	bIsInitialized = false;

	if (Level)
	{
		Level->bStaticComponentsRegisteredInStreamingManager = false;
	}
}

float FLevelRenderAssetManager::GetWorldTime() const
{
	return Level ? FRenderAssetStreamingManagerHelper::GetWorldTime(Level->GetWorld()) : 0.f;
}

void FLevelRenderAssetManager::SetAsStatic(FDynamicRenderAssetInstanceManager& DynamicComponentManager, const IPrimitiveComponent* Primitive, bool bIsConcurrent)
{
	check(Primitive);
	FRenderAssetOwnerStreamingState& StreamingState = Primitive->GetStreamingState();

	StreamingState.SetAttachedToStreamingManagerAsStatic(true);
	if (StreamingState.IsHandledByStreamingManagerAsDynamic())
	{
		if (bIsConcurrent)
		{
			SetAsLock.Lock();
			DynamicComponentManager.Remove(Primitive, nullptr);
			SetAsLock.Unlock();
		}
		else
		{
			DynamicComponentManager.Remove(Primitive, nullptr);
		}
		StreamingState.SetHandledByStreamingManagerAsDynamic(false);
	}
}


void FLevelRenderAssetManager::SetAsDynamic(FDynamicRenderAssetInstanceManager& DynamicComponentManager, FStreamingTextureLevelContext& LevelContext, const IPrimitiveComponent* Primitive, bool bIsConcurrent)
{
	check(Primitive);
	FRenderAssetOwnerStreamingState& StreamingState = Primitive->GetStreamingState();

	StreamingState.SetAttachedToStreamingManagerAsStatic(false);
	if (!StreamingState.IsHandledByStreamingManagerAsDynamic())
	{
		if (bIsConcurrent)
		{
			SetAsLock.Lock();
			DynamicComponentManager.Add(Primitive, LevelContext);
			SetAsLock.Unlock();
		}
		else
		{
			DynamicComponentManager.Add(Primitive, LevelContext);
		}
	}
}

void FLevelRenderAssetManager::IncrementalBuild(FDynamicRenderAssetInstanceManager& DynamicComponentManager, FStreamingTextureLevelContext& LevelContext, bool bForceCompletion, int64& NumStepsLeft)
{
	QUICK_SCOPE_CYCLE_COUNTER(FLevelRenderAssetManager_IncrementalBuild);
	check(Level);

	const float MaxTextureUVDensity = CVarStreamingMaxTextureUVDensity.GetValueOnAnyThread();

	switch (BuildStep)
	{
	case EStaticBuildStep::BuildTextureLookUpMap:
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FLevelRenderAssetManager::IncrementalBuild_BuildTextureLookUpMap);
		// Build the map to convert from a guid to the level index. 
		TextureGuidToLevelIndex.Reserve(Level->StreamingTextureGuids.Num());
		for (int32 TextureIndex = 0; TextureIndex < Level->StreamingTextureGuids.Num(); ++TextureIndex)
		{
			TextureGuidToLevelIndex.Add(Level->StreamingTextureGuids[TextureIndex], TextureIndex);
		}
		NumStepsLeft -= Level->StreamingTextureGuids.Num();
		BuildStep = EStaticBuildStep::ProcessActors;

		// Update the level context with the texture guid map. This is required in case the incremental build runs more steps.
		LevelContext.UpdateContext(GetCurrentMaterialQualityLevelChecked(), Level, &TextureGuidToLevelIndex);
		break;
	}
	case EStaticBuildStep::ProcessActors:
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FLevelRenderAssetManager::IncrementalBuild_ProcessActors);
		// All actors needs to be processed at once here because of the logic around bStaticComponentsRegisteredInStreamingManager.
		// All components must have either bHandledByStreamingManagerAsDynamic or bAttachedToStreamingManagerAsStatic set once bStaticComponentsRegisteredInStreamingManager gets set.
		// If any component gets created after, the logic in UPrimitiveComponent::CreateRenderState_Concurrent() will detect it as a new component and put it through the dynamic path.

		struct FProcessActorContext
		{
			TArray<const IPrimitiveComponent*> UnprocessedComponents{};
			int32 NumSteps = 0;
		};
		
		int32 MinBatchSize = 1;
		bool bIsParallelForAllowed = FRenderAssetStreamingManager::IsParallelForAllowedDuringIncrementalUpdate(Level->Actors.Num(), MinBatchSize);
		TArray<FProcessActorContext> Contexts;
		ParallelForWithTaskContext(TEXT("ProcessActors"), Contexts, Level->Actors.Num(), MinBatchSize, [this, &DynamicComponentManager, &LevelContext, bIsParallelForAllowed](FProcessActorContext& Context, int32 Index)
		{
			if (const AActor* Actor = Level->Actors[Index])
			{
				int32 NumSteps = 0;
				const bool bIsStaticActor = Actor->IsRootComponentStatic();
				Actor->ForEachComponent<UPrimitiveComponent>(false, [this, &DynamicComponentManager, &Context, &LevelContext, &NumSteps, &bIsStaticActor, bIsParallelForAllowed]
				(UPrimitiveComponent* Primitive)
				{
					if (bIsStaticActor && Primitive->Mobility == EComponentMobility::Static)
					{
						SetAsStatic(DynamicComponentManager, Primitive->GetPrimitiveComponentInterface(), bIsParallelForAllowed);
						Context.UnprocessedComponents.Push(Primitive->GetPrimitiveComponentInterface());
					}
					else
					{
						SetAsDynamic(DynamicComponentManager, LevelContext, Primitive->GetPrimitiveComponentInterface(), bIsParallelForAllowed);
					}
					++NumSteps;
				});
				Context.NumSteps += FMath::Max<int32>(NumSteps, 1);
			}
		}, bIsParallelForAllowed ? EParallelForFlags::None : EParallelForFlags::ForceSingleThread);
			
		for (FProcessActorContext& Context : Contexts)
		{
			UnprocessedComponents.Append(Context.UnprocessedComponents);
			NumStepsLeft -= Context.NumSteps;
		}

		NumStepsLeft -= (int64)FMath::Max<int32>(Level->Actors.Num(), 1);

		// Set a flag so that any further component added to the level gets handled as dynamic.
		Level->bStaticComponentsRegisteredInStreamingManager = true;

		BuildStep = EStaticBuildStep::ProcessComponents;
		break;
	}
	case EStaticBuildStep::ProcessComponents:
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FLevelRenderAssetManager::IncrementalBuild_ProcessComponents);
		const bool bLevelIsVisible = Level->bIsVisible;

		const int32 UnprocessedComponentCount = UnprocessedComponents.Num();
		const int32 ToProcessComponentCount = bForceCompletion ? UnprocessedComponentCount : FMath::Min<int32>(NumStepsLeft, UnprocessedComponentCount);
		int32 MinBatchSize = 1;
		if (FRenderAssetStreamingManager::IsParallelForAllowedDuringIncrementalUpdate(ToProcessComponentCount, MinBatchSize) && StaticInstances.CanAddComponent())
		{
			TArray<FRenderAssetInstanceState::FPreAddComponentPayload> Payloads;
			Payloads.SetNum(ToProcessComponentCount);
			// Because LevelContext is not thread-safe when ComponentBuildData is valid, we force not to use built data
			{
				LevelContext.SetForceNoUseBuiltData(true);
				ParallelFor(TEXT("PreAddComponents"), ToProcessComponentCount, MinBatchSize, [&](int32 Index)
				{
					FTaskTagScope Scope(ETaskTag::EParallelGameThread);
					const IPrimitiveComponent* Primitive = UnprocessedComponents[UnprocessedComponentCount - Index - 1];
					FRenderAssetInstanceState::PreAddComponent(Primitive, LevelContext, MaxTextureUVDensity, Payloads[Index]);
				});
				LevelContext.SetForceNoUseBuiltData(false);
			}

			StaticInstances.Add(Payloads, [this, &DynamicComponentManager, &LevelContext, bLevelIsVisible](const FRenderAssetInstanceState::FPreAddComponentPayload& Payload, EAddComponentResult AddResult)
			{
				check(AddResult != EAddComponentResult::Success);
				if (AddResult == EAddComponentResult::Fail && !bLevelIsVisible)
				{
					// Retry once the level becomes visible.
					PendingComponents.Add(Payload.Component);
				}
				else
				{
					// Here we also handle the case for Fail_UIDensityConstraint.
					SetAsDynamic(DynamicComponentManager, LevelContext, Payload.Component);
				}
			});

			NumStepsLeft -= ToProcessComponentCount;
			UnprocessedComponents.SetNum(UnprocessedComponentCount - ToProcessComponentCount, EAllowShrinking::No);
		}
		else
		{
			while ((bForceCompletion || NumStepsLeft > 0) && UnprocessedComponents.Num())
			{
				const IPrimitiveComponent* Primitive = UnprocessedComponents.Pop(EAllowShrinking::No);

				const EAddComponentResult AddResult = StaticInstances.Add(Primitive, LevelContext, MaxTextureUVDensity);
				if (AddResult == EAddComponentResult::Fail && !bLevelIsVisible)
				{
					// Retry once the level becomes visible.
					PendingComponents.Add(Primitive);
				}
				else if (AddResult != EAddComponentResult::Success)
				{
					// Here we also handle the case for Fail_UIDensityConstraint.
					SetAsDynamic(DynamicComponentManager, LevelContext, Primitive);
				}

				--NumStepsLeft;
			}
		}

		if (!UnprocessedComponents.Num())
		{
			UnprocessedComponents.Empty(); // Free the memory.
			BuildStep = EStaticBuildStep::NormalizeLightmapTexelFactors;
		}
		break;
	}
	case EStaticBuildStep::NormalizeLightmapTexelFactors:
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FLevelRenderAssetManager::IncrementalBuild_NormalizeLightmapTexelFactors);
		// Unfortunately, PendingInsertionStaticPrimtivComponents won't be taken into account here.
		StaticInstances.NormalizeLightmapTexelFactor();
		BuildStep = EStaticBuildStep::CompileElements;
		break;
	}
	case EStaticBuildStep::CompileElements:
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FLevelRenderAssetManager::IncrementalBuild_CompileElements);
		// Compile elements (to optimize runtime) for what is there.
		// PendingInsertionStaticPrimitives will be added after.
		NumStepsLeft -= (int64)StaticInstances.CompileElements();
		BuildStep = EStaticBuildStep::WaitForRegistration;
		break;
	}
	case EStaticBuildStep::WaitForRegistration:
		if (Level->bIsVisible)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(FLevelRenderAssetManager::IncrementalBuild_WaitForRegistration);
			// Remove unregistered component and resolve the bounds using the packed relative boxes.
			TArray<const IPrimitiveComponent*> RemovedPrimitives;
			NumStepsLeft -= (int64)StaticInstances.CheckRegistrationAndUnpackBounds(RemovedPrimitives);
			for (const IPrimitiveComponent* Primitive : RemovedPrimitives)
			{
				SetAsDynamic(DynamicComponentManager, LevelContext, Primitive);
			}

			NumStepsLeft -= (int64)PendingComponents.Num();

			// Reprocess the components that didn't have valid data.
			const int32 ToProcessComponentCount = PendingComponents.Num();
			int32 MinBatchSize = 1;
			if (FRenderAssetStreamingManager::IsParallelForAllowedDuringIncrementalUpdate(ToProcessComponentCount, MinBatchSize) && StaticInstances.CanAddComponent())
			{
				TArray<FRenderAssetInstanceState::FPreAddComponentPayload> Payloads;
				Payloads.SetNum(ToProcessComponentCount);
				// Because LevelContext is not thread-safe when ComponentBuildData is valid, we force not to use built data
				{
					LevelContext.SetForceNoUseBuiltData(true);
					ParallelFor(TEXT("PreAddComponents"), ToProcessComponentCount, MinBatchSize, [&](int32 Index)
					{
						FTaskTagScope Scope(ETaskTag::EParallelGameThread);
						FRenderAssetInstanceState::PreAddComponent(PendingComponents[Index], LevelContext, MaxTextureUVDensity, Payloads[Index]);
					});
					LevelContext.SetForceNoUseBuiltData(false);
				}

				StaticInstances.Add(Payloads, [this, &DynamicComponentManager, &LevelContext](const FRenderAssetInstanceState::FPreAddComponentPayload& Payload, EAddComponentResult AddResult)
				{
					check(AddResult != EAddComponentResult::Success);
					SetAsDynamic(DynamicComponentManager, LevelContext, Payload.Component);
				});
			}
			else
			{
				while (PendingComponents.Num())
				{
					const IPrimitiveComponent* Primitive = PendingComponents.Pop(EAllowShrinking::No);

					if (StaticInstances.Add(Primitive, LevelContext, MaxTextureUVDensity) != EAddComponentResult::Success)
					{
						SetAsDynamic(DynamicComponentManager, LevelContext, Primitive);
					}
				}
			}

			PendingComponents.Empty(); // Free the memory.
			TextureGuidToLevelIndex.Empty();
			BuildStep = EStaticBuildStep::Done;
		}
		break;
	default:
		break;
	}
}

bool FLevelRenderAssetManager::NeedsIncrementalBuild(int32 NumStepsLeftForIncrementalBuild) const
{
	check(Level);

	if (BuildStep == EStaticBuildStep::Done)
	{
		return false;
	}
	else if (Level->bIsVisible)
	{
		return true; // If visible, continue until done.
	}
	else // Otherwise, continue while there are incremental build steps available or we are waiting for visibility.
	{
		return BuildStep != EStaticBuildStep::WaitForRegistration && NumStepsLeftForIncrementalBuild > 0;
	}
}

void FLevelRenderAssetManager::IncrementalUpdate(
	FDynamicRenderAssetInstanceManager& DynamicComponentManager, 
	FRemovedRenderAssetArray& RemovedRenderAssets,
	int64& NumStepsLeftForIncrementalBuild, 
	float Percentage, 
	bool bUseDynamicStreaming) 
{
	check(Level);

	if (NeedsIncrementalBuild(NumStepsLeftForIncrementalBuild))
	{
		FStreamingTextureLevelContext LevelContext(GetCurrentMaterialQualityLevelChecked(), Level, &TextureGuidToLevelIndex);
		do
		{
			IncrementalBuild(DynamicComponentManager, LevelContext, Level->bIsVisible, NumStepsLeftForIncrementalBuild);
		}
		while (NeedsIncrementalBuild(NumStepsLeftForIncrementalBuild));
	}

	if (BuildStep == EStaticBuildStep::Done)
	{
		if (Level->bIsVisible)
		{
			bIsInitialized = true;
			// If the level is visible, update the bounds.
			StaticInstances.Refresh(Percentage);
		}
		else if (bIsInitialized)
		{
			// Mark all static textures for removal.
			for (auto It = StaticInstances.GetRenderAssetIterator(); It; ++It)
			{
				RemovedRenderAssets.Push(*It);
			}
			bIsInitialized = false;
		}
	}
}

void FLevelRenderAssetManager::NotifyLevelOffset(const FVector& Offset)
{
	if (BuildStep == EStaticBuildStep::Done)
	{
		// offset static primitives bounds
		StaticInstances.OffsetBounds(Offset);
	}
}

uint32 FLevelRenderAssetManager::GetAllocatedSize() const
{
	return StaticInstances.GetAllocatedSize() + 
		UnprocessedComponents.GetAllocatedSize() + 
		PendingComponents.GetAllocatedSize();
}

