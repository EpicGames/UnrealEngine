// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
TextureInstanceManager.h: Definitions of classes used for texture streaming.
=============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "StaticTextureInstanceManager.h"
#include "DynamicTextureInstanceManager.h"
#include "Templates/RefCounting.h"
#include "ContentStreaming.h"
#include "Engine/TextureStreamingTypes.h"
#include "Streaming/TextureStreamingHelpers.h"
#include "UObject/UObjectHash.h"
#include "Async/Mutex.h"

// The streaming data of a level.
class FLevelRenderAssetManager
{
public:

	FLevelRenderAssetManager(ULevel* InLevel, RenderAssetInstanceTask::FDoWorkTask& AsyncTask);

	ULevel* GetLevel() const { return Level; }

	FORCEINLINE bool HasRenderAssetReferences() const { return StaticInstances.HasRenderAssetReferences(); }

	// Remove the whole level. Optional list of textures or meshes referenced
	void Remove(FRemovedRenderAssetArray* RemovedRenderAssets);

	// Invalidate a component reference.

	FORCEINLINE void RemoveActorReferences(const AActor* Actor) {}

	void RemoveComponentReferences(const IPrimitiveComponent* Component, FRemovedRenderAssetArray& RemovedRenderAssets)
	{ 
		// Check everywhere as the mobility can change in game.
		StaticInstances.Remove(Component, &RemovedRenderAssets);
		UnprocessedComponents.RemoveSingleSwap(Component); 
		PendingComponents.RemoveSingleSwap(Component); 
	}

	const FStaticRenderAssetInstanceManager& GetStaticInstances() const { return StaticInstances; }

	float GetWorldTime() const;

	FORCEINLINE FRenderAssetInstanceAsyncView GetAsyncView() { return FRenderAssetInstanceAsyncView(StaticInstances.GetAsyncView(true)); }
	FORCEINLINE const FRenderAssetInstanceView* GetRawAsyncView() { return StaticInstances.GetAsyncView(false); }

	void IncrementalUpdate(FDynamicRenderAssetInstanceManager& DynamicManager, FRemovedRenderAssetArray& RemovedRenderAssets, int64& NumStepsLeftForIncrementalBuild, float Percentage, bool bUseDynamicStreaming);

	uint32 GetAllocatedSize() const;

	bool IsInitialized() const { return bIsInitialized; }
	bool HasBeenReferencedToStreamedTextures() const { return bHasBeenReferencedToStreamedTextures; }
	void SetReferencedToStreamedTextures() { bHasBeenReferencedToStreamedTextures = true; }

	void NotifyLevelOffset(const FVector& Offset);

private:

	ULevel* Level;

	bool bIsInitialized;
	bool bHasBeenReferencedToStreamedTextures;
	
	FStaticRenderAssetInstanceManager StaticInstances;

	/** Incremental build implementation. */

	enum class EStaticBuildStep : uint8
	{
		BuildTextureLookUpMap,
		ProcessActors,
		ProcessComponents,
		NormalizeLightmapTexelFactors,
		CompileElements,
		WaitForRegistration,
		Done,
	};

	// The current step of the incremental build.
	EStaticBuildStep BuildStep;
	// The components left to be processed in ProcessComponents
	TArray<const IPrimitiveComponent*> UnprocessedComponents;
	// The components that could not be processed by the incremental build.
	TArray<const IPrimitiveComponent*> PendingComponents;
	// Reversed lookup for ULevel::StreamingTextureGuids.
	TMap<FGuid, int32> TextureGuidToLevelIndex;
	// Used when SetAsStatic and/or SetAsDynamic are called concurrently
	UE::FMutex SetAsLock;

	bool NeedsIncrementalBuild(int32 NumStepsLeftForIncrementalBuild) const;
	void IncrementalBuild(FDynamicRenderAssetInstanceManager& DynamicComponentManager, FStreamingTextureLevelContext& LevelContext, bool bForceCompletion, int64& NumStepsLeft);

	FORCEINLINE_DEBUGGABLE void SetAsStatic(FDynamicRenderAssetInstanceManager& DynamicComponentManager, const IPrimitiveComponent* Primitive, bool bIsConcurrent = false);
	FORCEINLINE_DEBUGGABLE void SetAsDynamic(FDynamicRenderAssetInstanceManager& DynamicComponentManager, FStreamingTextureLevelContext& LevelContext, const IPrimitiveComponent* Primitive, bool bIsConcurrent = false);
};
