// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
RenderAssetUpdate.cpp: Base class of helpers to stream in and out texture/mesh LODs.
=============================================================================*/

#include "RenderAssetUpdate.h"
#include "Engine/StreamableRenderAsset.h"
#include "RenderingThread.h"
#include "Streaming/TextureStreamingHelpers.h"
#include "UObject/UObjectIterator.h"
#include "HAL/LowLevelMemStats.h"
#include "ProfilingDebugging/AssetMetadataTrace.h"
#include "UObject/Package.h"

float GStreamingFlushTimeOut = 3.00f;
static FAutoConsoleVariableRef CVarStreamingFlushTimeOut(
	TEXT("r.Streaming.FlushTimeOut"),
	GStreamingFlushTimeOut,
	TEXT("Time before we timeout when flushing streaming (default=3)"),
	ECVF_Default
);

static TAutoConsoleVariable<int32> CVarStreamingStressTestExtraAsyncLatency(
	TEXT("r.Streaming.StressTest.ExtraAsyncLatency"),
	0,
	TEXT("An extra latency in milliseconds for each async task when doing the stress test."),
	ECVF_Cheat);

ENGINE_API bool GEnableAssetUpdateAbandons = false;
static FAutoConsoleVariableRef CVarEnableAssetUpdateAbandons(
	TEXT("r.Streaming.EnableAssetUpdateAbandons"),
	GEnableAssetUpdateAbandons,
	TEXT("Enable asset update abandon support, enables calls to FRenderAssetUpdate::TryAbandon & FRenderAssetUpdate::OnAbandon which can then be used to optionally decouple Asset Update internals that stall/prevent objects from being GC'd"),
	ECVF_Default
);

std::atomic<int32> GRenderAssetStreamingSuspension = 0;

bool IsAssetStreamingSuspended()
{
	return GRenderAssetStreamingSuspension.load(std::memory_order_relaxed) > 0;
}

void SuspendRenderAssetStreaming()
{
	ensure(IsInGameThread());

	if (GRenderAssetStreamingSuspension.fetch_add(1, std::memory_order_relaxed) == 0)
	{
		bool bHasPendingStreamingRequest = false;

		// Wait for all assets to have their update lock unlocked. 
		TArray<UStreamableRenderAsset*> LockedAssets;
		for (TObjectIterator<UStreamableRenderAsset> It; It; ++It)
		{
			UStreamableRenderAsset* CurrentAsset = *It;
			if (CurrentAsset && CurrentAsset->IsStreamable() && CurrentAsset->HasPendingInitOrStreaming())
			{
				bHasPendingStreamingRequest = true;
				if (CurrentAsset->IsPendingStreamingRequestLocked())
				{
					LockedAssets.Add(CurrentAsset);
				}
			}
		}

		// If an asset stays locked for  GStreamingFlushTimeOut, 
		// we conclude there is a deadlock or that the object is never going to recover.

		float TimeLimit = GStreamingFlushTimeOut;

		while (LockedAssets.Num() && (TimeLimit > 0 || GStreamingFlushTimeOut <= 0))
		{
			FPlatformProcess::Sleep(RENDER_ASSET_STREAMING_SLEEP_DT);
			ENQUEUE_RENDER_COMMAND(FlushCommand)(
				[](FRHICommandListImmediate& RHICmdList)
				{
					RHICmdList.ImmediateFlush(EImmediateFlushType::FlushRHIThread);
				});
			FlushRenderingCommands();
			
			TimeLimit -= RENDER_ASSET_STREAMING_SLEEP_DT;

			for (int32 LockedIndex = 0; LockedIndex < LockedAssets.Num(); ++LockedIndex)
			{
				UStreamableRenderAsset* CurrentAsset = LockedAssets[LockedIndex];
				if (!CurrentAsset || !CurrentAsset->IsPendingStreamingRequestLocked())
				{
					LockedAssets.RemoveAtSwap(LockedIndex);
					--LockedIndex;
				}
			}
		}

		if (TimeLimit <= 0 && GStreamingFlushTimeOut > 0)
		{
			UE_LOGF(LogContentStreaming, Error, "SuspendRenderAssetStreaming timed out while waiting for asset:");
			for (int32 LockedIndex = 0; LockedIndex < LockedAssets.Num(); ++LockedIndex)
			{
				UStreamableRenderAsset* CurrentAsset = LockedAssets[LockedIndex];
				if (CurrentAsset)
				{
					if (!IsValid(CurrentAsset) || CurrentAsset->HasAnyFlags(RF_BeginDestroyed|RF_FinishDestroyed))
					{
						UE_LOGF(LogContentStreaming, Error, "	%ls", *CurrentAsset->GetFullName());
					}
					else
					{
						UE_LOGF(LogContentStreaming, Error, "	%ls (PendingKill)", *CurrentAsset->GetFullName());
					}
				}
			}
		}

		// At this point, no more rendercommands or IO requests can be generated before a call to ResumeRenderAssetStreamingRenderTasksInternal().

		if (bHasPendingStreamingRequest)
		{
			// Ensure any pending render command executes.
			ENQUEUE_RENDER_COMMAND(FlushCommand)(
				[](FRHICommandListImmediate& RHICmdList)
				{
					RHICmdList.ImmediateFlush(EImmediateFlushType::FlushRHIThread);
				});
			FlushRenderingCommands();
		}
	}
}

void ResumeRenderAssetStreaming()
{
	GRenderAssetStreamingSuspension.fetch_sub(1, std::memory_order_relaxed);
}

namespace FRenderAssetUpdateHelper
{
	bool TickRenderAssetUpdateForGarbageCollection(TRefCountPtr<FRenderAssetUpdate> RenderAssetUpdate)
	{
		// To avoid async tasks from timing-out the GC, we tick as Async to force completion if this is relevant.
		// This could lead the asset from releasing the PendingUpdate, which will be deleted once the async task completes.
		if (RenderAssetUpdate->GetRelevantThread() == FRenderAssetUpdate::TT_Async)
		{
			RenderAssetUpdate->Tick(FRenderAssetUpdate::TT_GameRunningAsync);
		}
		else
		{
			RenderAssetUpdate->Tick(GIsThreadedRendering ? FRenderAssetUpdate::TT_None : FRenderAssetUpdate::TT_Render);
		}
		return RenderAssetUpdate->IsCompleted();
	}
}

FRenderAssetUpdate::FRenderAssetUpdate(const UStreamableRenderAsset* InAsset)
	: ResourceState(InAsset->GetStreamableResourceState())
	, CurrentFirstLODIdx(InAsset->GetStreamableResourceState().ResidentFirstLODIdx())
	, PendingFirstLODIdx(InAsset->GetStreamableResourceState().RequestedFirstLODIdx())
	, ScheduledGTTasks(0)
	, ScheduledRenderTasks(0)
	, ScheduledAsyncTasks(0)
	, StreamableAsset(InAsset)
	, bIsCancelled(false)
	, bDeferExecution(false)
	, bSuccess(false)
	, TaskState(TS_Init)
{
	check(InAsset);
	if (!ensure(ResourceState.IsValidForStreamingRequest()))
	{
		bIsCancelled = true;
	}
}

FRenderAssetUpdate::~FRenderAssetUpdate()
{
	// Work must be done here because derived destructors have been called now and so derived members are invalid.
	check(TaskSynchronization.GetValue() == 0 && TaskState == TS_Done);
}

FReturnedRefCountValue FRenderAssetUpdate::Release() const
{
	uint32 NewValue = (uint32)NumRefs.Decrement();
	if (NewValue == 0)
	{
		if (ensure(TaskState == TS_Done) && !TaskSynchronization.GetValue())
		{
			delete this;
		}
		else
		{
			// Can't delete this object if some other system has some token to decrement.
			UE_LOGF(LogContentStreaming, Error, "RenderAssetUpdate is leaking (State=%d)", (int32)TaskState);
		}
	}
	return FReturnedRefCountValue{NewValue};
}

void FRenderAssetUpdate::Tick(EThreadType InCurrentThread)
{
	if (TaskState != TS_Done)
	{
		bool bIsLocked = true;

		// Should we do aynthing about FApp::ShouldUseThreadingForPerformance()? For example to prevent async thread from stalling game/renderthreads.
		// When the renderthread is the gamethread, don't lock if this is the renderthread to prevent stalling on low priority async tasks.
		if (InCurrentThread == TT_None || (InCurrentThread == TT_Render && !GIsThreadedRendering))
		{
			bIsLocked = CS.TryLock();
		}
		else if (InCurrentThread == TT_GameRunningAsync)
		{
			// When the GameThread tries to execute the async task, in GC, allow several attempts.
			bIsLocked = CS.TryLock();
			InCurrentThread = TT_Async;
		}
		else
		{
			CS.Lock();
		}
		
		if (bIsLocked)
		{
			// This will happens in PushTask() or when ScheduleRenderTask() ends up executing the command.
			const bool bWasAlreadyLocked = (TaskState == TS_Locked);
			TaskState = TS_Locked;

			ETaskState TickResult;
#if !UE_BUILD_SHIPPING
			TickResult = IterateInternalTickWithProfiling(InCurrentThread, bWasAlreadyLocked);
#else
			TickResult = IterateInternalTick(InCurrentThread, bWasAlreadyLocked);
#endif

			// We do this to prevent updating the TaskState while in the Lock.
			if (!bWasAlreadyLocked)
			{
				TaskState = TickResult;
			}
			CS.Unlock();
		}
	}
}

auto FRenderAssetUpdate::IterateInternalTick(EThreadType InCurrentThread, bool bWasAlreadyLocked) -> ETaskState
{
	ETaskState TickResult;
	do // Iterate as long as there is progress
	{
		// Only test for suspension the first time and in normal progress.
		// When cancelled, we want the update to complete without interruptions, allowing reference to be freed.
		TickResult = TickInternal(InCurrentThread, !bWasAlreadyLocked && !bIsCancelled);
	} while (TickResult == TS_Locked);

	return TickResult;
}

#if !UE_BUILD_SHIPPING
auto FRenderAssetUpdate::IterateInternalTickWithProfiling(EThreadType InCurrentThread, bool bWasAlreadyLocked) -> ETaskState
{
	ETaskState TickResult;
	if (StreamableAsset) // StreamableAsset can be nullptr because the asset update was abandoned (see r.Streaming.EnableAssetUpdateAbandons)
	{
		LLM_SCOPE_DYNAMIC_STAT_OBJECTPATH(StreamableAsset, ELLMTagSet::Assets);
		LLM_SCOPE_DYNAMIC_STAT_OBJECTPATH(StreamableAsset->GetClass(), ELLMTagSet::AssetClasses);
		UE_TRACE_METADATA_SCOPE_ASSET(StreamableAsset, StreamableAsset->GetClass());

		TickResult = IterateInternalTick(InCurrentThread, bWasAlreadyLocked);
	}
	else
	{
		TickResult = IterateInternalTick(InCurrentThread, bWasAlreadyLocked);
	}

	return TickResult;
}
#endif

class FRenderAssetUpdateTickGTTask
{
public:
	FORCEINLINE FRenderAssetUpdateTickGTTask(FRenderAssetUpdate* InUpdate)
		: PendingUpdate(InUpdate)
	{}

	static FORCEINLINE TStatId GetStatId()
	{
		RETURN_QUICK_DECLARE_CYCLE_STAT(FRenderAssetUpdateTickGTTask, STATGROUP_TaskGraphTasks);
	}

	static FORCEINLINE ENamedThreads::Type GetDesiredThread()
	{
		return ENamedThreads::GameThread;
	}

	static FORCEINLINE ESubsequentsMode::Type GetSubsequentsMode()
	{
		return ESubsequentsMode::FireAndForget;
	}

	void DoTask(ENamedThreads::Type CurThread, const FGraphEventRef& MyCompletionGraphEvent)
	{
		check(PendingUpdate);
		PendingUpdate->Tick(FRenderAssetUpdate::TT_GameThread);
		--PendingUpdate->ScheduledGTTasks;
	}

private:
	TRefCountPtr<FRenderAssetUpdate> PendingUpdate;
};

void FRenderAssetUpdate::ScheduleGTTask()
{
	check(TaskState == TS_Locked);

	if (IsInGameThread())
	{
		Tick(TT_GameThread);
	}
	else
	{
		// Notify that a tick is scheduled on the game thread
		++ScheduledGTTasks;

		TGraphTask<FRenderAssetUpdateTickGTTask>::CreateTask().ConstructAndDispatchWhenReady(this);
	}
}

void FRenderAssetUpdate::ScheduleRenderTask()
{
	check(TaskState == TS_Locked);

	// Notify that a tick is scheduled on the render thread.
	++ScheduledRenderTasks;
	// Increment refcount because we don't use a TRefCountPtr with ENQUEUE_RENDER_COMMAND.
	AddRef();

	ENQUEUE_RENDER_COMMAND(RenderAssetUpdateCommand)(
		[&](FRHICommandListImmediate&)
	{
		// Recompute the context has things might have changed!
		Tick(TT_Render);

		--ScheduledRenderTasks;

		// Decrement refcount because we don't use a TRefCountPtr with ENQUEUE_RENDER_COMMAND.
		Release();
	});
}

void FRenderAssetUpdate::ScheduleAsyncTask()
{
	check(TaskState == TS_Locked);

	// Notify that an async tick is scheduled.
	++ScheduledAsyncTasks;
	(new FAsyncMipUpdateTask(this))->StartBackgroundTask();
}

void FRenderAssetUpdate::FMipUpdateTask::DoWork()
{
	FTaskTagScope Scope(ETaskTag::EParallelGameThread);
	check(PendingUpdate.IsValid());

#if !UE_BUILD_SHIPPING
	const int32 ExtraSyncLatency = CVarStreamingStressTestExtraAsyncLatency.GetValueOnAnyThread();
	if (ExtraSyncLatency > 0)
	{
		// Slow down the async. Used to test GC issues.
		FPlatformProcess::Sleep(ExtraSyncLatency * .001f);
	}
#endif

	// Recompute the context has things might have changed!
	PendingUpdate->Tick(FRenderAssetUpdate::TT_Async);
	
	--PendingUpdate->ScheduledAsyncTasks;
}

FRenderAssetUpdate::ETaskState FRenderAssetUpdate::DoLock() 
{  
	CS.Lock(); 
	ETaskState PreviousTaskState = TaskState;
	TaskState = TS_Locked; 
	return PreviousTaskState;
}

void FRenderAssetUpdate::DoUnlock(ETaskState PreviousTaskState)
{
	TaskState = PreviousTaskState; 
	CS.Unlock();
}
