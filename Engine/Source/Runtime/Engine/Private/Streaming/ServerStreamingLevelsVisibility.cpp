// Copyright Epic Games, Inc. All Rights Reserved.

#include "Streaming/ServerStreamingLevelsVisibility.h"
#include "UObject/Package.h"
#include "Engine/LevelStreaming.h"
#include "Engine/Level.h"
#include "Engine/World.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ServerStreamingLevelsVisibility)

AServerStreamingLevelsVisibility::AServerStreamingLevelsVisibility(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bReplicates = false;
#if WITH_EDITORONLY_DATA
	bListedInSceneOutliner = false;
#endif
}

AServerStreamingLevelsVisibility* AServerStreamingLevelsVisibility::SpawnServerActor(UWorld* InWorld)
{
	const ENetMode NetMode = InWorld->GetNetMode();
	if (InWorld->IsGameWorld() && (NetMode == NM_DedicatedServer || NetMode == NM_ListenServer))
	{
		FActorSpawnParameters SpawnParams;
		SpawnParams.Name = AServerStreamingLevelsVisibility::StaticClass()->GetFName();
		SpawnParams.OverrideLevel = InWorld->PersistentLevel;
		if (AServerStreamingLevelsVisibility* ServerStreamingLevelVisibility = InWorld->SpawnActor<AServerStreamingLevelsVisibility>(AServerStreamingLevelsVisibility::StaticClass(), SpawnParams))
		{
			for (ULevelStreaming* StreamingLevel : InWorld->GetStreamingLevels())
			{
				if (StreamingLevel && StreamingLevel->IsLevelVisible())
				{
					ServerStreamingLevelVisibility->SetIsVisible(StreamingLevel, true);
				}
			}
			return ServerStreamingLevelVisibility;
		}
	}
	return nullptr;
}

bool AServerStreamingLevelsVisibility::Contains(const FName& InLevelPackageName) const
{
	return ServerVisibleStreamingLevels.Contains(InLevelPackageName);
}

ULevelStreaming* AServerStreamingLevelsVisibility::GetVisibleStreamingLevel(const FName& InPackageName) const
{
	const TWeakObjectPtr<ULevelStreaming>* StreamingLevel = ServerVisibleStreamingLevels.Find(InPackageName);
	return StreamingLevel ? StreamingLevel->Get() : nullptr;
}

void AServerStreamingLevelsVisibility::SetIsVisible(ULevelStreaming* InStreamingLevel, bool bInIsVisible)
{
	if (ensure(GetLocalRole() == ROLE_Authority))
	{
		if (InStreamingLevel)
		{
			const FName PackageName = InStreamingLevel->GetWorldAssetPackageFName();
			if (bInIsVisible)
			{
				ServerVisibleStreamingLevels.Add(PackageName, InStreamingLevel);
			}
			else
			{
				ServerVisibleStreamingLevels.Remove(PackageName);
			}
		}
	}
}
