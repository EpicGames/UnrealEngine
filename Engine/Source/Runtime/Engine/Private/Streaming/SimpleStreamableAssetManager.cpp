// Copyright Epic Games, Inc. All Rights Reserved.

#include "Streaming/SimpleStreamableAssetManager.h"

#include "Engine/StreamableRenderAsset.h"
#include "Engine/TextureStreamingTypes.h"
#include "Engine/Texture.h"
#include "Engine/SkeletalMesh.h"
#include "Engine/StaticMesh.h"
#include "PrimitiveSceneProxy.h"
#include "PrimitiveSceneInfo.h"
#include "ProfilingDebugging/CountersTrace.h"
#include "UObject/GarbageCollection.h"
#include "UObject/UObjectGlobals.h"
#include "SimpleStreamableAssetManagerLog.h"
#include "Streaming/TextureInstanceView.h"
#include "Streaming/TextureInstanceView.inl"
#include "Streaming/TextureStreamingHelpers.h"

DEFINE_LOG_CATEGORY(LogSimpleStreamableAssetManager);

TRACE_DECLARE_INT_COUNTER(RegisteredObjects, TEXT("StreamableAssets/SimpleStreamableAssetManager/RegisteredObjects"));
TRACE_DECLARE_INT_COUNTER(RegisteredAssets, TEXT("StreamableAssets/SimpleStreamableAssetManager/RegisteredAssets"));

TRACE_DECLARE_INT_COUNTER(AddedObjects, TEXT("StreamableAssets/SimpleStreamableAssetManager/AddedObjects"));
TRACE_DECLARE_INT_COUNTER(RemovedObjects, TEXT("StreamableAssets/SimpleStreamableAssetManager/RemovedObjects"));
TRACE_DECLARE_INT_COUNTER(UpdatedObjects, TEXT("StreamableAssets/SimpleStreamableAssetManager/UpdateObjects"));

// Bumps every time AssignPrimitiveHandle finds a live prior handle and recovers it (a missed
// Unregister upstream). Steady state should be 0; non-zero rate signals a regression.
TRACE_DECLARE_INT_COUNTER(SelfHealedRegistrations, TEXT("StreamableAssets/SimpleStreamableAssetManager/SelfHealedRegistrations"));



FSimpleStreamableAssetManager* FSimpleStreamableAssetManager::Instance = nullptr;
FSSAMAssetHandleTable          FSimpleStreamableAssetManager::AssetHandleTable;
FDelegateHandle                FSimpleStreamableAssetManager::PostGCDelegateHandle;
int32 FSimpleStreamableAssetManager::GUseSimpleStreamableAssetManager = 0;
int32 FSimpleStreamableAssetManager::GSimpleStreamableAssetManagerSparseArrayGrowSize = 64;
int32 FSimpleStreamableAssetManager::GSimpleStreamableAssetManagerEnsureAssetUniqueOnRegistration = 1;
int32 FSimpleStreamableAssetManager::GSimpleStreamableAssetManagerConsiderVisibility = 0;

FAutoConsoleVariableRef FSimpleStreamableAssetManager::CVarUseSimpleStreamableAssetManager(
TEXT("s.StreamableAssets.UseSimpleStreamableAssetManager"),
FSimpleStreamableAssetManager::GUseSimpleStreamableAssetManager,
TEXT("Whether to use FSimpleStreamableAssetManager.\n")
TEXT("If 0 (current default), the legacy texture streaming manager's per-UPrimitiveComponent path is used (component-based collection mostly on the game thread).\n")
TEXT("If non-zero, FSimpleStreamableAssetManager is enabled and integrates with SceneProxy to feed the system.\n")
TEXT("Read-only - must be set before FStreamingManagerCollection is constructed. Required by FastGeo Streaming."),
ECVF_SetByGameSetting | ECVF_ReadOnly
);

FAutoConsoleVariableRef FSimpleStreamableAssetManager::CVarUseSimpleStreamableAssetManagerSparseArrayGrowSize(
TEXT("s.StreamableAssets.SimpleStreamableAssetManager.SparseArrayGrowSize"),
FSimpleStreamableAssetManager::GSimpleStreamableAssetManagerSparseArrayGrowSize,
TEXT("The growth size of SparseArray used for tracking objects pointing specific assets"),
ECVF_SetByGameSetting
);

FAutoConsoleVariableRef FSimpleStreamableAssetManager::CVarUseSimpleStreamableAssetManagerEnsureAssetUniqueOnRegistration(
TEXT("s.StreamableAssets.SimpleStreamableAssetManager.SortAssetsOnRegistration"),
FSimpleStreamableAssetManager::GSimpleStreamableAssetManagerEnsureAssetUniqueOnRegistration,
TEXT("If true when object will be added, referenced assets will be sorted \n")
     TEXT("we will make sure we register asset only once with highest Texel Factor \n")
     TEXT("It will be beneficial only if multiple materials use same texture"),
ECVF_SetByGameSetting
);

// When disabled, streaming priority cannot distinguish on-screen from off-screen geometry.
// When enabled, the system tracks per-primitive last-rendered-time, which has a per-frame
// cost that scales with the visible primitive count. Profile before enabling.
FAutoConsoleVariableRef FSimpleStreamableAssetManager::CVarSimpleStreamableAssetManagerConsiderVisibility(
TEXT("s.StreamableAssets.SimpleStreamableAssetManager.ConsiderVisibility"),
FSimpleStreamableAssetManager::GSimpleStreamableAssetManagerConsiderVisibility,
TEXT("Whether the SimpleStreamableAssetManager should consider per-primitive visibility when computing screen-size streaming priority. Has a per-frame cost when enabled."),
ECVF_SetByGameSetting
);

void FSimpleStreamableAssetManager::Init()
{
	check(Instance == nullptr);
	Instance = new FSimpleStreamableAssetManager();

	// Drain producer queues after every GC pass. Commandlets and cookers don't tick the
	// streaming manager, so without this the RemovedAssetsRecords queue grows unbounded
	// and the handle table's monotonic NextIndex hits MaxSlots. The streaming tick remains
	// the primary drain when it runs; this is additive.
	check(!PostGCDelegateHandle.IsValid());
	PostGCDelegateHandle = FCoreUObjectDelegates::GetPostGarbageCollect().AddStatic(&FSimpleStreamableAssetManager::Process);
}

void FSimpleStreamableAssetManager::Shutdown()
{
	check(Instance != nullptr);

	if (PostGCDelegateHandle.IsValid())
	{
		FCoreUObjectDelegates::GetPostGarbageCollect().Remove(PostGCDelegateHandle);
		PostGCDelegateHandle.Reset();
	}

	delete Instance;
	Instance = nullptr;
}

FSSAMAssetHandle FSimpleStreamableAssetManager::AllocateAssetHandle()
{
	check(IsEnabled());
	return AssetHandleTable.Allocate();
}

int32 FSimpleStreamableAssetManager::GetAssetRegistrationIndex(FSSAMAssetHandle Handle)
{
	return AssetHandleTable.GetIndex(Handle);
}

void FSimpleStreamableAssetManager::SetAssetRegistrationIndex(FSSAMAssetHandle Handle, int32 NewIndex)
{
	check(IsEnabled());
	AssetHandleTable.SetIndex(Handle, NewIndex);
}

int32 FSimpleStreamableAssetManager::ReleaseAssetHandle(FSSAMAssetHandle Handle)
{
	return AssetHandleTable.Release(Handle);
}

bool FSimpleStreamableAssetManager::IsAssetRegistered(FSSAMAssetHandle Handle)
{
	if (!IsEnabled())
	{
		return false;
	}
	return AssetHandleTable.GetIndex(Handle) != INDEX_NONE;
}

void FSimpleStreamableAssetManager::Process()
{
	if (IsEnabled())
	{
		check(Instance != nullptr);
		FScopedLock ScopedLock(GetCriticalSection(), IsEnabled());
		Instance->UpdateTask_Async();
	}
}

void FSimpleStreamableAssetManager::ProcessFallbackRegistrations_GameThread()
{
	if (IsEnabled())
	{
		SCOPED_NAMED_EVENT(FSimpleStreamableAssetManager_ProcessFallbackRegistrations, FColor::Silver);
		/* FSimpleStreamableAssetManager intends to use scene proxy to register streamable render assets,
		 * but the interface is not implemented for all render proxy implementations and/or licensee code
		 * if you notice that this scope takes significant time in your project please consider implementing
		 * the interface GetStreamableRenderAssetInfo and set the flag bImplementsStreamableAssetGathering
		 * Code below is fallback to gathering assets from the primitive component that can be done safely
		 * in generic case only from the GameThread
		 */

		check(IsInGameThread());
		check(Instance != nullptr);
		// ExtractAll drains the entire FallbackComponentRegistration queue each frame; there is
		// currently no per-frame cap. A burst of RecreateRenderState across many components
		// (e.g., level streaming, material reload, or PSO fallback recreate flushes) is handled
		// in a single GT batch rather than time-sliced. If this becomes a hitch source, introduce
		// a per-frame cap with leftover entries re-pushed for the next tick.
		TArray<FFallbackEntry> FallbackEntries;
		Instance->FallbackComponentRegistration.ExtractAll(FallbackEntries);

		FStreamingTextureLevelContext LevelContext(GetCurrentMaterialQualityLevelChecked());
		LevelContext.SetForceNoUseBuiltData(true);

		for (const FFallbackEntry& Entry : FallbackEntries)
		{
			UPrimitiveComponent* Component = Entry.Component.Get();
			if (!Component)
			{
				// Component was destroyed before drain; nothing to register.
				continue;
			}
			// Only process this entry if it still represents the component's active registration.
			// If the component was unregistered (handle cleared to default) or superseded by a later
			// Register call (e.g., RecreateRenderState replaced the handle with a newer one),
			// the entry's captured handle no longer matches and we skip -- preventing both stale
			// registrations and the double-register from duplicate queue entries.
			if (Component->GetSSAMPrimitiveHandle() != Entry.Handle)
			{
				continue;
			}
			TArray<FStreamingRenderAssetPrimitiveInfo> Assets = LevelContext.GetStreamingRenderAssetInfoWithNULLRemoval(Component->GetPrimitiveComponentInterface());
			if (Assets.IsEmpty())
			{
				// Nothing streamable to register. The component's handle stays with it until the
				// normal unregister path; pushing an empty FRegister here would just be trimmed at
				// drain time (Record.RegisteredAssets.Num() == 0) -- skip it upstream.
				continue;
			}
			PushRegisterRecord(FRegister(Component, Assets));
		}
	}
}

void FSimpleStreamableAssetManager::EnqueueOrReleaseCapturedHandle(FSSAMAssetHandle CapturedHandle, UStreamableRenderAsset* DebugAsset)
{
	if (!CapturedHandle.IsValid())
	{
		return;
	}

	if (Instance == nullptr)
	{
		// Post-Shutdown / pre-Init: no drain to feed. Direct release is the only option;
		// there is no slot data to reclaim because the SSAM arrays are gone.
		ReleaseAssetHandle(CapturedHandle);
		return;
	}

	Instance->RemovedAssetsRecords.Push(FRemovedAssetRecord{
		.AssetHandle = CapturedHandle
#if SIMPLE_STREAMABLE_ASSET_MANAGER_ALLOW_DEBUG_POINTER
		, .DebugStreamableRenderAsset = reinterpret_cast<UPTRINT>(DebugAsset)
#endif
	});
}

void FSimpleStreamableAssetManager::UnregisterAsset(UStreamableRenderAsset* InAsset)
{
	if (!IsEnabled())
	{
		return;
	}

	check(IsInGameThread()); // asset teardown is GT-only

	if (Instance == nullptr)
	{
		// SSAM was shut down (or hasn't been constructed yet). No record queues to drain and
		// no concurrent SSAM readers, so we don't need the pre-allocate-and-swap dance --
		// just clear the asset's field and release whatever handle it still holds. Allocating
		// a replacement here would leak its slot since there's no Instance to drain it through.
		const FSSAMAssetHandle Captured = InAsset->SSAMAssetHandle.Exchange(FSSAMAssetHandle{});
		ReleaseAssetHandle(Captured);
		return;
	}

	// Pre-allocate the replacement handle so the asset's field never holds an empty value visible
	// to concurrent readers. The atomic exchange swaps OldHandle -> NewHandle in one op; workers
	// that read Asset->GetSSAMAssetHandle() during this call see either the old or the new handle,
	// never the intermediate empty state. Mirrors the old code's MakeShared(INDEX_NONE) replacement
	// and preserves the invariant that an alive asset always has a valid handle.
	const FSSAMAssetHandle NewHandle = AllocateAssetHandle();
	const FSSAMAssetHandle CapturedHandle = InAsset->SSAMAssetHandle.Exchange(NewHandle);

	if (!CapturedHandle.IsValid())
	{
		// Asset wasn't registered; the fresh handle we just allocated is unused -- release it.
		ReleaseAssetHandle(NewHandle);
		return;
	}

	EnqueueOrReleaseCapturedHandle(CapturedHandle, InAsset);
}

void FSimpleStreamableAssetManager::QueueRegisteredAssetHandleRelease(FSSAMAssetHandle Handle)
{
	if (!IsEnabled())
	{
		return;
	}

	check(IsInGameThread()); // asset teardown is GT-only

	// Caller (e.g., UStreamableRenderAsset::BeginDestroy) has already cleared the asset's
	// field. The asset is being destroyed, so no replacement handle is needed.
	EnqueueOrReleaseCapturedHandle(Handle, nullptr);
}

uint32 FSimpleStreamableAssetManager::GetAllocatedSize()
{
	constexpr uint32 StaticSize = sizeof(Instance) + sizeof(GUseSimpleStreamableAssetManager) + sizeof(CVarUseSimpleStreamableAssetManager); //-V568
	if (IsEnabled())
	{
		check(Instance != nullptr);
		FScopedLock SimpleScopedLock(GetCriticalSection(), IsEnabled());
		return StaticSize + Instance->GetAllocatedSize_Impl();
	}
	return StaticSize;
}

void FSimpleStreamableAssetManager::GetAssetReferenceBounds(const UStreamableRenderAsset* Asset, TArray<FBox>& AssetBoxes)
{
	check(Instance != nullptr);
	FScopedLock SimpleScopedLock(GetCriticalSection(), IsEnabled());
	Instance->GetAssetReferenceBounds_Impl(Asset, AssetBoxes);
}

float FSimpleStreamableAssetManager::GetStreamingScaleFactor(const FPrimitiveSceneProxy* InPrimitiveSceneProxy, const FMatrix& LocalToWorld)
{
	return InPrimitiveSceneProxy->CanApplyStreamableRenderAssetScaleFactor() ? LocalToWorld.GetMaximumAxisScale() : 1.f;
}

void FSimpleStreamableAssetManager::UpdateBoundSizes(
	const TArray<FStreamingViewInfo>& ViewInfos,
	const TArray<FStreamingViewInfoExtra, TInlineAllocator<4>>& ViewInfoExtras,
	float LastUpdateTime,
	const FRenderAssetStreamingSettings& Settings)
{
	check(Instance != nullptr);
	Instance->UpdateBoundSizes_Impl(ViewInfos, ViewInfoExtras, LastUpdateTime, Settings);
}

void FSimpleStreamableAssetManager::GetRenderAssetScreenSize(
		EStreamableRenderAssetType AssetType,
		FSSAMAssetHandle Handle,
		float& MaxSize,
		float& MaxSize_VisibleOnly,
		int32& MaxNumForcedLODs,
		const float MaxAssetSize,
		const int32 MaxAllowedMip,
		const TCHAR* LogPrefix)
{
	check(Instance != nullptr);
	const int32 AssetIndex = GetAssetRegistrationIndex(Handle);
	if (AssetIndex == INDEX_NONE)
	{
		return;
	}
	Instance->GetRenderAssetScreenSize_Impl(AssetType, AssetIndex, MaxSize, MaxSize_VisibleOnly, MaxNumForcedLODs, MaxAssetSize, MaxAllowedMip, LogPrefix);
}

void FSimpleStreamableAssetManager::GetDistanceAndRange(const FUpdate& Record, float& MinDistanceSq, float& MinRangeSq, float& MaxRangeSq)
{
	// RenderAssetInstanceBounds is Object Bounds since there is no parent support
	MinDistanceSq = FMath::Max<float>(0.0f, Record.MinDistance - Record.ObjectBounds.SphereRadius);
	MinDistanceSq *= MinDistanceSq;
	MinRangeSq = FMath::Max<float>(0, Record.MinDistance);
	MinRangeSq *= MinRangeSq;
	MaxRangeSq = FMath::Max<float>(0, Record.MaxDistance);
	MaxRangeSq *= MaxRangeSq;
}

// Asset UObject derefs below (IsStreamable, IsA<>, GetLODGroupForStreaming, GetSSAMAssetHandle) run on the
// caller's thread. Concurrent GC is held off by the FGCScopeGuard taken in
// FSimpleStreamableAssetManager::Register before constructing this FRegister, so every
// UStreamableRenderAsset reachable via InPrimitive->GetStreamableRenderAssetInfo at call entry remains
// a valid UObject for the entire duration of this constructor -- regardless of caller thread context
// (GT, ParallelGT, or true worker).
//
// The caller must pass a proxy whose reachable assets are live UObjects at call entry; the guard
// covers the mid-call window where GC could otherwise free them.
//
// For future caching work: IsStreamable() at StreamableRenderAsset.h reads StreamingIndex, which is
// runtime-mutable via LinkStreaming / UnlinkStreaming (e.g. Texture2D::CreateResource triggers UnlinkStreaming
// on live assets). Caching bIsStreamable in FMaterialCachedTexturesSamplingInfo or similar would require
// invalidation hooks on Link/Unlink events for every cached texture.
FSimpleStreamableAssetManager::FRegister::FRegister(const FPrimitiveSceneProxy* InPrimitive, const FMatrix& InRenderMatrix, const FBoxSphereBounds& InBounds)
	: FUpdate(
		InPrimitive,
		InBounds,
		FSimpleStreamableAssetManager::GetStreamingScaleFactor(InPrimitive, InRenderMatrix),
		InPrimitive->GetMinDrawDistance(),
		InPrimitive->GetMaxDrawDistance(),
		InPrimitive->GetPrimitiveSceneInfo()->LastRenderTime,
		InPrimitive->IsForceMipStreaming())
{
	TArray<FStreamingRenderAssetPrimitiveInfo> Assets;

	InPrimitive->GetStreamableRenderAssetInfo(InBounds, Assets);

	if (Assets.Num() > 0)
	{
		// Ignore not needed assets
		RegisteredAssets.Reserve(Assets.Num());
		for (FStreamingRenderAssetPrimitiveInfo& Info : Assets)
		{
			if (Info.RenderAsset && Info.RenderAsset->IsStreamable())
			{
#if DO_CHECK
				ensure(Info.TexelFactor >= 0.f
					|| Info.RenderAsset->IsA<UStaticMesh>()
					|| Info.RenderAsset->IsA<USkeletalMesh>()
					|| (Info.RenderAsset->IsA<UTexture>() && Info.RenderAsset->GetLODGroupForStreaming() == TEXTUREGROUP_Terrain_Heightmap));
#endif
				// Otherwise check that everything is setup right. If the component is not yet registered, then the bound data is irrelevant.
				if (bForceMipStreaming || Info.CanBeStreamedByDistance(true) || Info.TexelFactor < 0.f)
				{
					RegisteredAssets.Add(FRegisteredAsset{
						.AssetHandle = Info.RenderAsset->GetSSAMAssetHandle(),
						.AssetPrimitiveInfo = MoveTemp(Info)});
				}
			}
		}
	}
}

FSimpleStreamableAssetManager::FRegister::FRegister(const UPrimitiveComponent* InPrimitive, const TArray<FStreamingRenderAssetPrimitiveInfo>& Assets)
	: FUpdate(
		InPrimitive,
		InPrimitive->GetBounds(),
		InPrimitive->GetStreamingScale(),
		InPrimitive->GetMinDrawDistance(),
		InPrimitive->CachedMaxDrawDistance,
		InPrimitive->GetLastRenderTimeOnScreen(),
		InPrimitive->bForceMipStreaming)
{
	if (Assets.Num() > 0)
	{
		// Ignore not needed assets
		RegisteredAssets.Reserve(Assets.Num());
		for (const FStreamingRenderAssetPrimitiveInfo& Info : Assets)
		{
			if (Info.RenderAsset && Info.RenderAsset->IsStreamable())
			{
#if DO_CHECK
				ensure(Info.TexelFactor >= 0.f
					|| Info.RenderAsset->IsA<UStaticMesh>()
					|| Info.RenderAsset->IsA<USkeletalMesh>()
					|| (Info.RenderAsset->IsA<UTexture>() && Info.RenderAsset->GetLODGroupForStreaming() == TEXTUREGROUP_Terrain_Heightmap));
#endif
				// Otherwise check that everything is setup right. If the component is not yet registered, then the bound data is irrelevant.
				if (bForceMipStreaming || Info.CanBeStreamedByDistance(true) || Info.TexelFactor < 0.f)
				{
					RegisteredAssets.Add(FRegisteredAsset{
						.AssetHandle = Info.RenderAsset->GetSSAMAssetHandle(),
						.AssetPrimitiveInfo = Info});
				}
			}
		}
	}
}

void FSimpleStreamableAssetManager::Register(IPrimitiveComponent* ComponentInterface, FPrimitiveSceneProxy* SceneProxy, const FMatrix& RenderMatrix, const FBoxSphereBounds& WorldBounds)
{
	if (!IsEnabled())
	{
		return;
	}

	check(SceneProxy);
	UPrimitiveComponent* PrimitiveComponent = ComponentInterface ? ComponentInterface->GetUObject<UPrimitiveComponent>() : nullptr;
	// When writing to a UPrimitiveComponent's field, require GT or parallel-GT.
	// FScene::BatchAddPrimitivesInternal can be called from either context.
	// Proxy-only callers (PrimitiveComponent == nullptr) bypass this check and may run on true worker
	// threads (e.g. FastGeo's async render state creation). Those callers must hold the assets they
	// reference via some UObject ownership chain at call entry; SSAM blocks GC during the FRegister
	// ctor below to prevent concurrent collection from invalidating those references mid-gather.
	// See the block comment at FRegister::FRegister(const FPrimitiveSceneProxy*, ...).
	check(!PrimitiveComponent || IsInGameThread() || IsInParallelGameThread());
	const bool bGatheringSupported = SceneProxy->IsSupportingStreamableRenderAssetsGathering();

	// Primitive-shape filter: a non-gathering proxy with no UPrimComp can't be registered through
	// either SSAM path -- no UPrimitiveComponent for the GT fallback queue and no gathering impl
	// for the worker path. Legacy streaming is also keyed off UPrimitiveComponent, so it doesn't
	// cover this case either: asset streaming for such proxies is effectively a coverage gap.
	// Warn once so the gap is visible; proper fix is to add GetStreamableRenderAssetInfo support
	// to the proxy type.
	if (!bGatheringSupported && !PrimitiveComponent)
	{
		static bool bLoggedOnce = false;
		UE_CLOGF(!bLoggedOnce, LogSimpleStreamableAssetManager, Warning, "proxy %ls (resource %ls) (interface %ls) is not registerable (no UPrimitiveComponent and does not implement GetStreamableRenderAssetInfo). Asset streaming for this proxy type will not be driven by FSimpleStreamableAssetManager or legacy streaming. Further warnings suppressed.", *SceneProxy->GetOwnerName().ToString(), *SceneProxy->GetResourceName().ToString(), ComponentInterface ? *ComponentInterface->GetFullName() : TEXT("nullptr"));
		bLoggedOnce = true;
		return;
	}

	const FSSAMPrimitiveHandle NewPrimitiveHandle = Instance->AllocatePrimitiveHandle();
	AssignPrimitiveHandle(SceneProxy, PrimitiveComponent, NewPrimitiveHandle);

	if (bGatheringSupported)
	{
		// Block GC for the duration of the FRegister ctor. The ctor walks the proxy's
		// GetStreamableRenderAssetInfo result and dereferences the returned UStreamableRenderAsset
		// pointers (IsStreamable, IsA<>, GetSSAMAssetHandle). FGCScopeGuard prevents a concurrent
		// GC mark/sweep from invalidating those references mid-gather. The guard is a no-op cost
		// on GC-pinned threads (GT, ParallelGT, render thread) and a brief wait on true workers
		// only when GC is currently active. Released as soon as the ctor returns; the queue push
		// itself does not deref any UObject.
		FGCScopeGuard GCLock;
		PushRegisterRecord(FRegister(SceneProxy, RenderMatrix, WorldBounds));
	}
	else
	{
		// Defer GetStreamingRenderAssetInfo to GT via the fallback queue. Reachable only with a
		// UPrimitiveComponent + no gathering impl on the proxy; filters above rule out everything else.
		check(PrimitiveComponent);
		Instance->FallbackComponentRegistration.Push(FFallbackEntry{TWeakObjectPtr<UPrimitiveComponent>{PrimitiveComponent}, NewPrimitiveHandle });
	}
}

void FSimpleStreamableAssetManager::Unregister(IPrimitiveComponent* ComponentInterface, FPrimitiveSceneProxy* SceneProxy)
{
	if (!IsEnabled())
	{
		return;
	}
	// Render-thread BatchRemovePrimitives commands can pump after Shutdown has cleared Instance;
	// drop the unregister rather than dereferencing the null manager.
	if (!Instance)
	{
		return;
	}
	check(SceneProxy);

	UPrimitiveComponent* PrimitiveComponent = ComponentInterface ? ComponentInterface->GetUObject<UPrimitiveComponent>() : nullptr;
	// When writing to a UPrimitiveComponent's field, require GT or parallel-GT.
	check(!PrimitiveComponent || IsInGameThread() || IsInParallelGameThread());

	// Proxy was never registered (caller guard or filter rejection during Register).
	if (!SceneProxy->GetSSAMPrimitiveHandle().IsValid())
	{
		return;
	}

	// Capture the handle and clear both proxy + component fields in one op, then push the unregister
	// record. SceneProxy is passed as the debug-inspection object carried through the record.
	PushUnregisterRecord(FUnregister{ ClearPrimitiveHandle(SceneProxy, PrimitiveComponent), SceneProxy });
}

void FSimpleStreamableAssetManager::RecoverMissedUnregister(FSSAMPrimitiveHandle Handle)
{
	if (!IsEnabled() || !Handle.IsValid())
	{
		return;
	}
	// Drop any handle whose slot is no longer live: stale (generation already bumped by some
	// other path) or post-Shutdown (Instance gone). Both mean "nothing to clean up" -- pushing
	// the record would either be a no-op at drain or hit the check inside PushUnregisterRecord.
	if (!Instance || !Instance->IsPrimitiveHandleLive(Handle))
	{
		return;
	}
	// Null debug pointer: deferred OnCommit dispatch can outlive the original caller's
	// UObject (in-place reconstruction reuses the address for a different object).
	PushUnregisterRecord(FUnregister{ Handle, nullptr });
}

void FSimpleStreamableAssetManager::AssignPrimitiveHandle(FPrimitiveSceneProxy* Proxy, UPrimitiveComponent* Component, FSSAMPrimitiveHandle PrimitiveHandle)
{
	// Install the new handle in one atomic op, capturing whatever the field held before. The
	// captured value covers all three pre-states (default / stale / live) uniformly, so no
	// separate stale-cleanup pass is needed.
	//
	// Per-field atomicity, NOT pair-wise: each Exchange below is atomic with respect to its
	// individual field, but the two stores are sequenced and a reader observing both fields
	// between them can see the proxy at the new handle while the component still holds the
	// prior value (or vice versa). No current consumer cross-checks both fields concurrently;
	// ClearPrimitiveHandle's aliasing assertion is the only equality check, and it runs at
	// teardown serialized against this install path, not concurrently with it.
	const FSSAMPrimitiveHandle PriorProxyHandle = Proxy->SSAMPrimitiveHandle.Exchange(PrimitiveHandle);
	if (Component)
	{
		const FSSAMPrimitiveHandle PriorComponentHandle = Component->SSAMPrimitiveHandle.Exchange(PrimitiveHandle);
		// Aliasing invariant: Component side either matches Proxy or was never written
		// (prior Register was proxy-only). Mismatch on a live handle is a genuine bug.
		check(!PriorComponentHandle.IsValid() || PriorComponentHandle == PriorProxyHandle);
	}

	// Self-heal: a live prior handle means Register was called twice without an intervening
	// Unregister. Stale priors don't need self-heal (some other path already released the slot);
	// default priors mean a clean first-time registration. Without this heal the missed
	// Unregister would silently leak its registration (orphaned ObjectIndex, duplicated asset
	// reference bound, stale handle slot) in shipping.
	if (PriorProxyHandle.IsValid() && !PriorProxyHandle.IsStale())
	{
		TRACE_COUNTER_INCREMENT(SelfHealedRegistrations);
		UE_LOGF(LogSimpleStreamableAssetManager, Error, "FSimpleStreamableAssetManager::AssignPrimitiveHandle: proxy '%ls' already has a live SSAMPrimitiveHandle. A Register was issued without a prior Unregister -- self-healing the previous registration.", *Proxy->GetOwnerName().ToString());
		PushUnregisterRecord(FUnregister{ PriorProxyHandle, Proxy });
	}
}

FSSAMPrimitiveHandle FSimpleStreamableAssetManager::ClearPrimitiveHandle(FPrimitiveSceneProxy* Proxy, UPrimitiveComponent* Component)
{
	// Capture-and-clear in one atomic op so a concurrent reader can't see the proxy field
	// as live after we've snapshotted OldProxyHandle.
	const FSSAMPrimitiveHandle OldProxyHandle = Proxy->SSAMPrimitiveHandle.Exchange(FSSAMPrimitiveHandle{});
	if (Component)
	{
		const FSSAMPrimitiveHandle OldComponentHandle = Component->SSAMPrimitiveHandle.Exchange(FSSAMPrimitiveHandle{});
		// Aliasing invariant: Component side either matches Proxy or was never written
		// (prior Register was proxy-only). Mismatch on a live handle is a genuine bug.
		check(!OldComponentHandle.IsValid() || OldComponentHandle == OldProxyHandle);
	}
	return OldProxyHandle;
}

void FSimpleStreamableAssetManager::PushRegisterRecord(FRegister&& Record)
{
	check(Instance != nullptr);
	Instance->RegisterRecords.Push(MoveTemp(Record));
}

void FSimpleStreamableAssetManager::PushUnregisterRecord(FUnregister&& Record)
{
	check(Instance != nullptr);
	Instance->UnregisterRecords.Push(MoveTemp(Record));
}

void FSimpleStreamableAssetManager::PushUpdateRecord(FUpdate&& Record)
{
	check(Instance != nullptr);
	Instance->UpdateRecords.Push(MoveTemp(Record));
}

void FSimpleStreamableAssetManager::PushUpdateLastRenderTimeRecord(FUpdateLastRenderTime&& Record)
{
	check(Instance != nullptr);
	Instance->UpdateLastRenderTimeRecords.Push(MoveTemp(Record));
}

void FSimpleStreamableAssetManager::UpdateStreamingState(const FPrimitiveSceneProxy* Proxy, const FMatrix& LocalToWorld, const FBoxSphereBounds& WorldBounds)
{
	if (!IsEnabled() || !Proxy->GetSSAMPrimitiveHandle().IsValid())
	{
		return;
	}
	PushUpdateRecord(FUpdate{
		Proxy,
		WorldBounds,
		GetStreamingScaleFactor(Proxy, LocalToWorld),
		Proxy->GetMinDrawDistance(),
		Proxy->GetMaxDrawDistance(),
		Proxy->GetPrimitiveSceneInfo()->GetSceneData()->GetLastRenderTimeOnScreen(),
		Proxy->IsForceMipStreaming()
	});
}

void FSimpleStreamableAssetManager::UpdateLastRenderTime(const FPrimitiveSceneProxy* Proxy, float CurrentWorldTime)
{
	if (!IsEnabled() || !ShouldConsiderVisibility() || !Proxy->GetSSAMPrimitiveHandle().IsValid())
	{
		return;
	}
	PushUpdateLastRenderTimeRecord(FUpdateLastRenderTime{ Proxy, CurrentWorldTime });
}

void FSimpleStreamableAssetManager::GetAssetReferenceBounds_Impl(const UStreamableRenderAsset* Asset, TArray<FBox>& AssetBoxes) const
{
	const FSSAMAssetHandle AssetHandle = Asset->GetSSAMAssetHandle();
	if (!AssetHandle.IsValid())
	{
		return;
	}
	const int32 AssetIndex = GetAssetRegistrationIndex(AssetHandle);

	if (AssetIndex != INDEX_NONE)
	{
		const TSimpleSparseArray<FAssetBoundElement>& AssetElements = AssetIndexToBounds4Index[AssetIndex];

		AssetBoxes.Reserve(AssetElements.Num());
		
		for (const FAssetBoundElement& Element : AssetElements.GetSparseView())
		{
			const int32 ObjectRegistrationIndex = Element.ObjectRegistrationIndex;
			if (ObjectRegistrationIndex != INDEX_NONE)
			{
				const FBoxSphereBounds Bounds = GetBounds(ObjectRegistrationIndex);
				AssetBoxes.Add(Bounds.GetBox());
			}
		}
	}
}

uint32 FSimpleStreamableAssetManager::GetAllocatedSize_Impl() const
{
	uint32 AllocatedSize = sizeof(FSimpleStreamableAssetManager);

	AllocatedSize += ObjectUsedIndices.GetAllocatedSize();
	for (const TArray<FAssetRecord>& Assets : ObjectRegistrationIndexToAssetProperty)
	{
		AllocatedSize += Assets.GetAllocatedSize();
	}
	
	AllocatedSize += ObjectBounds4.GetAllocatedSize();

	AllocatedSize += AssetUsedIndices.GetAllocatedSize();

	for (const TSimpleSparseArray<FAssetBoundElement>& AssetBoundElements : AssetIndexToBounds4Index)
	{
		AllocatedSize += AssetBoundElements.GetAllocatedSize();
	}
	AllocatedSize += AssetIndexToHandle.GetAllocatedSize();

	AllocatedSize += BoundsViewInfos.GetAllocatedSize();

	// Consumer-side staging arrays drained from the producer queues each Process() cycle.
	AllocatedSize += Pending_UpdateRecords.GetAllocatedSize();
	AllocatedSize += Pending_UpdateLastRenderTimeRecords.GetAllocatedSize();
	AllocatedSize += Pending_RegisterRecords.GetAllocatedSize();
	AllocatedSize += Pending_UnregisterRecords.GetAllocatedSize();
	AllocatedSize += Pending_RemovedAssetRecords.GetAllocatedSize();

	// Producer-side lock-free queues (TConsumeAllMpmcQueue node allocations).
	AllocatedSize += FallbackComponentRegistration.GetAllocatedSize();
	AllocatedSize += RemovedAssetsRecords.GetAllocatedSize();
	AllocatedSize += RegisterRecords.GetAllocatedSize();
	AllocatedSize += UnregisterRecords.GetAllocatedSize();
	AllocatedSize += UpdateRecords.GetAllocatedSize();
	AllocatedSize += UpdateLastRenderTimeRecords.GetAllocatedSize();

	AllocatedSize += AssetHandleTable.GetAllocatedSize();
	AllocatedSize += PrimitiveHandleTable.GetAllocatedSize();

	return AllocatedSize;
}

void FSimpleStreamableAssetManager::UpdateTask_Async()
{
	SCOPED_NAMED_EVENT(FSimpleStreamableAssetManager_UpdateTask_Async, FColor::Silver);
		
	const int32 LastRegisteredObjectCount = RegisteredObjectCount;
	const int32 LastRegisteredAssetsCount = UsedAssetIndices;

	check(Pending_UpdateRecords.IsEmpty());
	check(Pending_UpdateLastRenderTimeRecords.IsEmpty());
	check(Pending_RegisterRecords.IsEmpty());
	check(Pending_UnregisterRecords.IsEmpty());
	check(Pending_RemovedAssetRecords.IsEmpty());
	
	{
		// Here, the order of extraction of the records is important.
		// This consumer is non-blocking; producers may push from other threads.
		SCOPED_NAMED_EVENT(FSimpleStreamableAssetManager_MoveData, FColor::Silver);
		UpdateRecords.ExtractAll(Pending_UpdateRecords);
		UpdateLastRenderTimeRecords.ExtractAll(Pending_UpdateLastRenderTimeRecords);
		RegisterRecords.ExtractAll(Pending_RegisterRecords);
		UnregisterRecords.ExtractAll(Pending_UnregisterRecords);
		RemovedAssetsRecords.ExtractAll(Pending_RemovedAssetRecords);
	}
	
	/** Assign a indice to each proxy **/
	{
		SCOPED_NAMED_EVENT(FSimpleStreamableAssetManager_AssignObjectIndex, FColor::Silver);
		
		// Trim records with no streamable assets or already-assigned / stale handles before reserving,
		// so the reservation matches the records that will actually consume slots.
		for (int32 RecordIndex = Pending_RegisterRecords.Num() - 1; RecordIndex >= 0; --RecordIndex)
		{
			const FRegister& Record = Pending_RegisterRecords[RecordIndex];
			const int32 CurrentIndex = GetPrimitiveRegistrationIndex(Record.PrimitiveHandle);
			// CurrentIndex == INDEX_NONE means either "not yet assigned" (normal) or "stale handle"
			// (handle was Released before this record drained). IsValid disambiguates.
			const bool bLive = IsPrimitiveHandleLive(Record.PrimitiveHandle);
			if (Record.RegisteredAssets.Num() == 0 || !bLive || CurrentIndex != INDEX_NONE)
			{
				Pending_RegisterRecords.RemoveAtSwap(RecordIndex, EAllowShrinking::No);
			}
		}

		const int32 NeedToReserve = RegisteredObjectCount + Pending_RegisterRecords.Num() - MaxObjects;
		if (NeedToReserve > 0)
		{
			SCOPED_NAMED_EVENT(FSimpleStreamableAssetManager_Resize, FColor::Silver);
			ObjectUsedIndices.Add(false, NeedToReserve);
			ObjectRegistrationIndexToAssetProperty.AddZeroed(NeedToReserve);
			MaxObjects += NeedToReserve;

			// We store 4 Object bounds in one entry
			const int32 MaxObject4BoundsNeeded = MaxObjects / 4 + int32(MaxObjects % 4 != 0);
			const int32 Object4BoundsNeedToReserve = MaxObject4BoundsNeeded - ObjectBounds4.Num();
			if (Object4BoundsNeedToReserve > 0)
			{
				ObjectBounds4.AddDefaulted(Object4BoundsNeedToReserve);
			}
		}

		for (const FRegister& Record : Pending_RegisterRecords)
		{
			// Cheap index assignment for proxy
			const int32 ObjectIndex = ObjectUsedIndices.FindAndSetFirstZeroBit(FreeObjectIndexHint);
			check(ObjectIndex != INDEX_NONE);
			FreeObjectIndexHint = ObjectIndex + 1;
			++RegisteredObjectCount;
			SetPrimitiveRegistrationIndex(Record.PrimitiveHandle, ObjectIndex);
		}
	}
	
	{
		SCOPED_NAMED_EVENT(FSimpleStreamableAssetManager_Process, FColor::Silver);

		{
			SCOPED_NAMED_EVENT(FSimpleStreamableAssetManager_Unregister, FColor::Silver);
			for (FUnregister& Record : Pending_UnregisterRecords)
			{
				UnregisterRecord(Record);
			}
		}
		
		{
			SCOPED_NAMED_EVENT(FSimpleStreamableAssetManager_Register, FColor::Silver);
			for (FRegister& Record : Pending_RegisterRecords)
			{
				RegisterRecord(Record);
			}
		}

		{
			SCOPED_NAMED_EVENT(FSimpleStreamableAssetManager_Update, FColor::Silver);
			for (FUpdate& Record : Pending_UpdateRecords)
			{
				UpdateRecord(Record);
			}
		}

		{
			SCOPED_NAMED_EVENT(FSimpleStreamableAssetManager_UpdateLastRenderTime, FColor::Silver);
			for (FUpdateLastRenderTime& Record : Pending_UpdateLastRenderTimeRecords)
			{
				UpdateRecord(Record);
			}
		}

		{
			SCOPED_NAMED_EVENT(FSimpleStreamableAssetManager_RemoveAssets, FColor::Silver);
			for (FRemovedAssetRecord& Record : Pending_RemovedAssetRecords)
			{
				// Release captures the slot's last RegistrationIndex in one atomic op, bumps the
				// generation (invalidating any remaining copies of the handle), and pushes the
				// slot back to the free list.
				const int32 AssetIndex = ReleaseAssetHandle(Record.AssetHandle);
				if (AssetIndex != INDEX_NONE)
				{
					AssetUsedIndices[AssetIndex] = false;
					--UsedAssetIndices;
					FreeAssetIndexHint = FMath::Min(FreeAssetIndexHint, AssetIndex);
					AssetIndexToBounds4Index[AssetIndex].Empty();
					AssetIndexToHandle[AssetIndex].Reset();
				}
			}
		}
	}
	
	if (LastRegisteredObjectCount != RegisteredObjectCount)
	{
		TRACE_COUNTER_SET(RegisteredObjects, RegisteredObjectCount);
	}

	if (LastRegisteredAssetsCount != UsedAssetIndices)
	{
		TRACE_COUNTER_SET(RegisteredAssets, UsedAssetIndices);
	}

#if COUNTERSTRACE_ENABLED
	const int32 Removed = Pending_UnregisterRecords.Num();
	TRACE_COUNTER_SET(RemovedObjects, Removed);

	const int32 Added = Pending_RegisterRecords.Num();
	TRACE_COUNTER_SET(AddedObjects, Added);

	const int32 Updates = Pending_UpdateRecords.Num();
	TRACE_COUNTER_SET(UpdatedObjects, Updates);
#endif

	Pending_UpdateRecords.Reset();
	Pending_UpdateLastRenderTimeRecords.Reset();
	Pending_RegisterRecords.Reset();
	Pending_UnregisterRecords.Reset();
	Pending_RemovedAssetRecords.Reset();
}

void FSimpleStreamableAssetManager::RegisterRecord(FRegister& Record)
{
	const int32 ObjectIndex = GetPrimitiveRegistrationIndex(Record.PrimitiveHandle);

	// Defensive: UpdateTask_Async's trim + assign passes guarantee a valid ObjectIndex here.
	if (ObjectIndex == INDEX_NONE)
	{
		return;
	}

	TArray<FRegisteredAsset>& RegisteredAssets = Record.RegisteredAssets;
	int32 AssetCount = RegisteredAssets.Num();

	if (GSimpleStreamableAssetManagerEnsureAssetUniqueOnRegistration && AssetCount > 0)
	{
		// Sort by Texture to merge duplicate texture entries.
		// Then sort by TexelFactor
		RegisteredAssets.Sort([](const FRegisteredAsset& Lhs, const FRegisteredAsset& Rhs)
		{
			const FStreamingRenderAssetPrimitiveInfo& LhsInfo = Lhs.AssetPrimitiveInfo;
			const FStreamingRenderAssetPrimitiveInfo& RhsInfo = Rhs.AssetPrimitiveInfo;

			if (LhsInfo.RenderAsset == RhsInfo.RenderAsset)
			{
				return LhsInfo.TexelFactor > RhsInfo.TexelFactor;
			}
			return LhsInfo.RenderAsset < RhsInfo.RenderAsset;
		});

		int32 ProcessedIndex = 0;
		int32 EmplaceIndex = 1;

		for (int32 Index = 1; Index < RegisteredAssets.Num(); Index++)
		{
			const FStreamingRenderAssetPrimitiveInfo& CurInfo = RegisteredAssets[Index].AssetPrimitiveInfo;
			FStreamingRenderAssetPrimitiveInfo& ProcessedInfo = RegisteredAssets[ProcessedIndex].AssetPrimitiveInfo;

			// We found new asset 
			if (ProcessedInfo.RenderAsset != CurInfo.RenderAsset)
			{
				if (EmplaceIndex != Index) // We need to fill the gap in array
				{
					RegisteredAssets[EmplaceIndex] = MoveTemp(RegisteredAssets[Index]);
				}
				++ProcessedIndex;
				++EmplaceIndex;
			}
			// For landscape entries negative TexelFactor values are used we need to ensure we get min while sorted max 
			else if (ProcessedInfo.TexelFactor < 0 && ProcessedInfo.RenderAsset == CurInfo.RenderAsset)
			{
				ProcessedInfo.TexelFactor = FMath::Min(ProcessedInfo.TexelFactor, CurInfo.TexelFactor);
			}
		}

		AssetCount = EmplaceIndex;
	}
		
	if (AssetCount > 0)
	{
		float MinDistanceSq = 0, MinRangeSq = 0, MaxRangeSq = FLT_MAX;
        GetDistanceAndRange(Record, MinDistanceSq, MinRangeSq, MaxRangeSq);
        SetBounds(ObjectIndex, Record.ObjectBounds, PackedRelativeBox_Identity, Record.StreamingScaleFactor, Record.LastRenderedTime, Record.ObjectBounds.Origin, MinDistanceSq, MinRangeSq, MaxRangeSq);

		const bool bForceLOD = Record.bForceMipStreaming;
		AddRenderAssetElements(TArrayView<FRegisteredAsset>(RegisteredAssets.GetData(), AssetCount), ObjectIndex, bForceLOD);
	}
}

void FSimpleStreamableAssetManager::UpdateRecord(FUpdate& Record)
{
	const int32 ObjectIndex = GetPrimitiveRegistrationIndex(Record.PrimitiveHandle);
	if (ObjectIndex != INDEX_NONE) // Filter out updates for not-registered / stale proxies
	{
		float MinDistanceSq = 0, MinRangeSq = 0, MaxRangeSq = FLT_MAX;
		GetDistanceAndRange(Record, MinDistanceSq, MinRangeSq, MaxRangeSq);
		SetBounds(ObjectIndex, Record.ObjectBounds, PackedRelativeBox_Identity, Record.StreamingScaleFactor, Record.LastRenderedTime, Record.ObjectBounds.Origin, MinDistanceSq, MinRangeSq, MaxRangeSq);
	}
}

void FSimpleStreamableAssetManager::UpdateRecord(FUpdateLastRenderTime& Record)
{
	const int32 ObjectIndex = GetPrimitiveRegistrationIndex(Record.PrimitiveHandle);
	if (ObjectIndex != INDEX_NONE) // Filter out updates for not-registered / stale proxies
	{
		TrySetLastRenderTime(ObjectIndex, Record.LastRenderedTime);
	}
}

void FSimpleStreamableAssetManager::UnregisterRecord(FUnregister& Record)
{
	// Release captures the slot's last RegistrationIndex in one atomic op, bumps the generation
	// (invalidating any remaining copies of the handle), and pushes the slot to the free list.
	const int32 ObjectIndex = ReleasePrimitiveHandle(Record.PrimitiveHandle);
	if (ObjectIndex != INDEX_NONE)
	{
		RemoveRenderAssetElements(ObjectIndex);
	}
}

void FSimpleStreamableAssetManager::TrySetLastRenderTime(int32 BoundsIndex, float LastRenderedTime)
{
	const int32 ObjectBounds4Index = BoundsIndex / 4;
	if (ObjectBounds4.IsValidIndex(ObjectBounds4Index))
	{
		ObjectBounds4[ObjectBounds4Index].UpdateLastRenderTime(BoundsIndex % 4, LastRenderedTime);
	}
}

void FSimpleStreamableAssetManager::SetBounds(int32 BoundsIndex, const FBoxSphereBounds& Bounds, uint32 PackedRelativeBox, float StreamingScaleFactor, float LastRenderTime, const FVector4& RangeOrigin, float MinDistanceSq, float MinRangeSq, float MaxRangeSq)
{
	const int32 ObjectBounds4Index = BoundsIndex / 4;

	if (ObjectBounds4Index >= ObjectBounds4.Num())
	{
		check(ObjectBounds4Index == ObjectBounds4.Num());
		ObjectBounds4.Add(FBounds4{});
	}
	// We store 4 Objects in one entry
	ObjectBounds4[ObjectBounds4Index].Set(BoundsIndex % 4, Bounds, PackedRelativeBox, StreamingScaleFactor, LastRenderTime, RangeOrigin, MinDistanceSq, MinRangeSq, MaxRangeSq);
}

FBoxSphereBounds FSimpleStreamableAssetManager::GetBounds(int32 BoundsIndex) const
{
	FBoxSphereBounds Bounds(ForceInitToZero);
	const int32 ObjectBounds4Index = BoundsIndex / 4;
	const int32 ObjectBounds4Offset = BoundsIndex % 4;

	check(BoundsIndex >= 0 && ObjectBounds4Index < ObjectBounds4.Num());

	const FBounds4& TheBounds4 = ObjectBounds4[ObjectBounds4Index];

	Bounds.Origin.X = TheBounds4.OriginX[ObjectBounds4Offset];
	Bounds.Origin.Y = TheBounds4.OriginY[ObjectBounds4Offset];
	Bounds.Origin.Z = TheBounds4.OriginZ[ObjectBounds4Offset];

	Bounds.BoxExtent.X = TheBounds4.ExtentX[ObjectBounds4Offset];
	Bounds.BoxExtent.Y = TheBounds4.ExtentY[ObjectBounds4Offset];
	Bounds.BoxExtent.Z = TheBounds4.ExtentZ[ObjectBounds4Offset];

	Bounds.SphereRadius = Bounds.BoxExtent.Length();

	return Bounds;
}

void FSimpleStreamableAssetManager::AddRenderAssetElements(const TArrayView<FRegisteredAsset>& RegisteredAssets, int32 ObjectRegistrationIndex, bool bForceMipStreaming)
{
	TSet<FAssetRecord> ObjectAssets;
	ObjectAssets.Reserve(RegisteredAssets.Num());
	for (const FRegisteredAsset& RegisteredAsset : RegisteredAssets)
	{
		UStreamableRenderAsset* RenderAsset = RegisteredAsset.AssetPrimitiveInfo.RenderAsset;
		if (RenderAsset == nullptr)
		{
			continue;
		}
		
		const int32 AssetIndex = [&]
		{
			const int32 CurrentIndex = GetAssetRegistrationIndex(RegisteredAsset.AssetHandle);
			if (CurrentIndex != INDEX_NONE)
			{
				return CurrentIndex;
			}

			if (UsedAssetIndices == AssetUsedIndices.Num())
			{
				AssetUsedIndices.Add(false, 32);
			}

			const int32 NewAssetIndex = AssetUsedIndices.FindAndSetFirstZeroBit(FreeAssetIndexHint);
			check(NewAssetIndex != INDEX_NONE);
			FreeAssetIndexHint = NewAssetIndex + 1;
			++UsedAssetIndices;
			SetAssetRegistrationIndex(RegisteredAsset.AssetHandle, NewAssetIndex);

			return NewAssetIndex;
		}();

		check(AssetIndex != INDEX_NONE);

		if (AssetIndex >= AssetIndexToBounds4Index.Num())
		{
			check(AssetIndex == AssetIndexToBounds4Index.Num());
			AssetIndexToBounds4Index.Add({});
			AssetIndexToHandle.Add(RegisteredAsset.AssetHandle);
		}
		else if (!AssetIndexToHandle[AssetIndex].IsValid())
		{
			// Re-populate after a previous slot reclamation freed this index
			AssetIndexToHandle[AssetIndex] = RegisteredAsset.AssetHandle;
		}
		
		const int32 AssetElementIndex = AssetIndexToBounds4Index[AssetIndex].Add(FAssetBoundElement{
						.ObjectRegistrationIndex = ObjectRegistrationIndex,
						.TexelFactor = RegisteredAsset.AssetPrimitiveInfo.TexelFactor,
						.bForceLOD = bForceMipStreaming,
						.bAffectedByComponentScale = RegisteredAsset.AssetPrimitiveInfo.bAffectedByComponentScale });

		ObjectAssets.Add(FAssetRecord{ .AssetHandle = RegisteredAsset.AssetHandle, .AssetElementIndex = AssetElementIndex
#if SIMPLE_STREAMABLE_ASSET_MANAGER_ALLOW_DEBUG_POINTER
			, .DebugStreamableRenderAsset = reinterpret_cast<UPTRINT>(RenderAsset)
#endif
			});
	}

	ObjectRegistrationIndexToAssetProperty[ObjectRegistrationIndex] = ObjectAssets.Array();
}

void FSimpleStreamableAssetManager::RemoveRenderAssetElements(int32 ObjectRegistrationIndex)
{
	check(ObjectRegistrationIndex != INDEX_NONE)
	--RegisteredObjectCount;
	ObjectUsedIndices[ObjectRegistrationIndex] = false;			
	FreeObjectIndexHint = FMath::Min(FreeObjectIndexHint, ObjectRegistrationIndex);
	
	TArray<FAssetRecord>& ObjectAssets = ObjectRegistrationIndexToAssetProperty[ObjectRegistrationIndex];
	
	for (const FAssetRecord& Asset : ObjectAssets)
	{
		// Resolve the asset's current registration index via the handle. Returns INDEX_NONE
		// for any stale handle (asset was unregistered while this record was still in flight;
		// the slot may have been recycled to a different asset by now). Skipping the record
		// avoids corrupting the recycled slot's element bindings -- erasing an unrelated
		// primitive's entry on the new asset would lose its streaming contribution silently.
		const int32 AssetIndex = GetAssetRegistrationIndex(Asset.AssetHandle);
		if (AssetIndex == INDEX_NONE)
		{
			continue;
		}
		const int32 AssetElementIndex = Asset.AssetElementIndex;
		check(AssetElementIndex != INDEX_NONE);

		AssetIndexToBounds4Index[AssetIndex].Reset(AssetElementIndex);

		// When all objects referencing an asset are removed, fully reclaim the asset slot to
		// prevent unbounded memory growth when assets outlive the objects that reference them.
		// Stale records are filtered above by GetAssetRegistrationIndex returning INDEX_NONE,
		// so reaching here implies a live registration. The AssetUsedIndices guard is retained
		// as defense-in-depth against any future ordering surprises.
		if (AssetIndexToBounds4Index[AssetIndex].Num() == 0 && AssetUsedIndices[AssetIndex])
		{
			AssetIndexToBounds4Index[AssetIndex].Empty();
			AssetUsedIndices[AssetIndex] = false;
			--UsedAssetIndices;
			FreeAssetIndexHint = FMath::Min(FreeAssetIndexHint, AssetIndex);
			if (AssetIndexToHandle[AssetIndex].IsValid())
			{
				// Clear the index via the handle table; any remaining copies of the handle will
				// observe INDEX_NONE on their next GetIndex until the asset itself is released.
				SetAssetRegistrationIndex(AssetIndexToHandle[AssetIndex], INDEX_NONE);
				AssetIndexToHandle[AssetIndex].Reset();
			}
		}
	}
	ObjectAssets.Reset();
#if DO_CHECK
	// We store 4 Objects in one Bounds entry, there is no real need for clearing the entry
	// we will clean it only for ease of debugging if checks enabled 
	ObjectBounds4[ObjectRegistrationIndex / 4].Clear(ObjectRegistrationIndex % 4);
#endif
}

void FSimpleStreamableAssetManager::GetRenderAssetScreenSize_Impl(
		EStreamableRenderAssetType AssetType,
		const int32 InAssetIndex,
		float& MaxSize,
		float& MaxSize_VisibleOnly,
		int32& MaxNumForcedLODs,
		const float MaxAssetSize,
		const int32 MaxAllowedMip,
		const TCHAR* LogPrefix) const
{
	if (AssetType != EStreamableRenderAssetType::Texture || MaxSize_VisibleOnly < MaxAssetSize || LogPrefix)
	{
		// Asset might be registered, but it might not be yet used by any proxy
		if (InAssetIndex != INDEX_NONE)
		{
			const TSimpleSparseArray<FAssetBoundElement>& AssetBoundElements = AssetIndexToBounds4Index[InAssetIndex];
						
			for (const FAssetBoundElement& AssetBoundElement : AssetBoundElements.GetSparseView())	
			{
				const int32 ObjectRegistrationIndex =  AssetBoundElement.ObjectRegistrationIndex; 

				if (ObjectRegistrationIndex != INDEX_NONE)
				{
					const FBoundsViewInfo& BoundsViewInfo = BoundsViewInfos[ObjectRegistrationIndex];
			
					const bool bApplyComponentScale = (AssetType == EStreamableRenderAssetType::Texture) && AssetBoundElement.bAffectedByComponentScale;
					const float TexelFactor = bApplyComponentScale ? AssetBoundElement.TexelFactor * BoundsViewInfo.ComponentScale : AssetBoundElement.TexelFactor;
					const bool bForcedLODs = AssetBoundElement.bForceLOD;
					FRenderAssetInstanceAsyncView::ProcessElement(AssetType, BoundsViewInfo, TexelFactor, bForcedLODs, MaxSize, MaxSize_VisibleOnly, MaxNumForcedLODs);

					if (LogPrefix)
					{
						FBoxSphereBounds Bounds = GetBounds(ObjectRegistrationIndex);
						FRenderAssetInstanceView::OutputToLog(Bounds, ObjectRegistrationIndex, ObjectBounds4, TexelFactor, bForcedLODs, BoundsViewInfo.MaxNormalizedSize, BoundsViewInfo.MaxNormalizedSize_VisibleOnly, LogPrefix);
					}
					else if (MaxSize_VisibleOnly >= MaxAssetSize || MaxNumForcedLODs >= MaxAllowedMip)
					{
						return;
					}
				}
			}
		}
	}
}

void FSimpleStreamableAssetManager::UpdateBoundSizes_Impl(
	const TArray<FStreamingViewInfo>& ViewInfos,
	const TArray<FStreamingViewInfoExtra, TInlineAllocator<4>>& ViewInfoExtras,
	float LastUpdateTime,
	const FRenderAssetStreamingSettings& Settings)
{
	SCOPED_NAMED_EVENT(FSimpleStreamableAssetManager_UpdateBounds, FColor::Silver);
	
	constexpr float MaxTexelFactor = 1.0f; 
	float MaxLevelRenderAssetScreenSize = 0.0f;
	FRenderAssetInstanceAsyncView::UpdateBoundSizes(
		ViewInfos,
		ViewInfoExtras,
		LastUpdateTime,
		MaxTexelFactor,
		Settings,
		ObjectBounds4,
		BoundsViewInfos,
		MaxLevelRenderAssetScreenSize
	);
}