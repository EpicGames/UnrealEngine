// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
StaticMeshUpdate.cpp: Helpers to stream in and out static mesh LODs.
=============================================================================*/

#include "Streaming/StaticMeshUpdate.h"
#include "HAL/PlatformFile.h"
#include "RenderUtils.h"
#include "Streaming/TextureStreamingHelpers.h"
#include "Serialization/MemoryReader.h"
#include "StaticMeshResources.h"
#include "Streaming/RenderAssetUpdate.inl"
#include "ContentStreaming.h"
#include "RHIResourceReplace.h"
#include "ProfilingDebugging/IoStoreTrace.h"
#include "ProfilingDebugging/RealtimeGPUProfiler.h"

int32 GStreamingMaxReferenceChecks = 2;
static FAutoConsoleVariableRef CVarStreamingMaxReferenceChecksBeforeStreamOut(
	TEXT("r.Streaming.MaxReferenceChecksBeforeStreamOut"),
	GStreamingMaxReferenceChecks,
	TEXT("Number of times the engine wait for references to be released before forcing streamout. (default=2)"),
	ECVF_Default
);

// Instantiate TRenderAssetUpdate for FStaticMeshUpdateContext
template class TRenderAssetUpdate<FStaticMeshUpdateContext>;

static constexpr uint32 GStaticMeshMaxNumResourceUpdatesPerLOD = 14;
static constexpr uint32 GStaticMeshMaxNumResourceUpdatesPerBatch = (MAX_STATIC_MESH_LODS - 1) * GStaticMeshMaxNumResourceUpdatesPerLOD;

FStaticMeshUpdateContext::FStaticMeshUpdateContext(const UStaticMesh* InMesh, EThreadType InCurrentThread)
	: Mesh(InMesh)
	, CurrentThread(InCurrentThread)
{
	check(InMesh);
	checkSlow(InCurrentThread != FStaticMeshUpdate::TT_Render || IsInRenderingThread());
	// #TODO RenderData under a const UStaticMesh should stay const
	RenderData = const_cast<UStaticMesh*>(InMesh)->GetRenderData();
	if (RenderData)
	{
		LODResourcesView = TArrayView<FStaticMeshLODResources*>(RenderData->LODResources.GetData() + InMesh->GetStreamableResourceState().AssetLODBias, InMesh->GetStreamableResourceState().MaxNumLODs);
	}
}

FStaticMeshUpdateContext::FStaticMeshUpdateContext(const UStreamableRenderAsset* InMesh, EThreadType InCurrentThread)
#if UE_BUILD_SHIPPING
	: FStaticMeshUpdateContext(static_cast<const UStaticMesh*>(InMesh), InCurrentThread)
#else
	: FStaticMeshUpdateContext(Cast<UStaticMesh>(InMesh), InCurrentThread)
#endif
{}

FStaticMeshUpdate::FStaticMeshUpdate(const UStaticMesh* InMesh)
	: TRenderAssetUpdate<FStaticMeshUpdateContext>(InMesh)
{
}

void FStaticMeshStreamIn::FIntermediateBuffers::CreateFromCPUData(FRHICommandListBase& RHICmdList, FStaticMeshLODResources& LODResource)
{
	FStaticMeshVertexBuffers& VBs = LODResource.VertexBuffers;
	TangentsVertexBuffer = VBs.StaticMeshVertexBuffer.CreateTangentsRHIBuffer(RHICmdList);
	TexCoordVertexBuffer = VBs.StaticMeshVertexBuffer.CreateTexCoordRHIBuffer(RHICmdList);
	PositionVertexBuffer = VBs.PositionVertexBuffer.CreateRHIBuffer(RHICmdList);
	ColorVertexBuffer = VBs.ColorVertexBuffer.CreateRHIBuffer(RHICmdList);
	IndexBuffer = LODResource.IndexBuffer.CreateRHIBuffer(RHICmdList);
	DepthOnlyIndexBuffer = LODResource.DepthOnlyIndexBuffer.CreateRHIBuffer(RHICmdList);

	if (LODResource.AdditionalIndexBuffers)
	{
		ReversedIndexBuffer = LODResource.AdditionalIndexBuffers->ReversedIndexBuffer.CreateRHIBuffer(RHICmdList);
		ReversedDepthOnlyIndexBuffer = LODResource.AdditionalIndexBuffers->ReversedDepthOnlyIndexBuffer.CreateRHIBuffer(RHICmdList);
		WireframeIndexBuffer = LODResource.AdditionalIndexBuffers->WireframeIndexBuffer.CreateRHIBuffer(RHICmdList);
	}
}

void FStaticMeshStreamIn::FIntermediateBuffers::TransferBuffers(FStaticMeshLODResources& LODResource, FRHIResourceReplaceBatcher& Batcher)
{
	FStaticMeshVertexBuffers& VBs = LODResource.VertexBuffers;
	VBs.StaticMeshVertexBuffer.InitRHIForStreaming(TangentsVertexBuffer, TexCoordVertexBuffer, Batcher);
	VBs.PositionVertexBuffer.InitRHIForStreaming(PositionVertexBuffer, Batcher);
	VBs.ColorVertexBuffer.InitRHIForStreaming(ColorVertexBuffer, Batcher);
	LODResource.IndexBuffer.InitRHIForStreaming(IndexBuffer, Batcher);
	LODResource.DepthOnlyIndexBuffer.InitRHIForStreaming(DepthOnlyIndexBuffer, Batcher);

	if (LODResource.AdditionalIndexBuffers)
	{
		LODResource.AdditionalIndexBuffers->ReversedIndexBuffer.InitRHIForStreaming(ReversedIndexBuffer, Batcher);
		LODResource.AdditionalIndexBuffers->ReversedDepthOnlyIndexBuffer.InitRHIForStreaming(ReversedDepthOnlyIndexBuffer, Batcher);
		LODResource.AdditionalIndexBuffers->WireframeIndexBuffer.InitRHIForStreaming(WireframeIndexBuffer, Batcher);
	}
}

#if RHI_RAYTRACING

void FStaticMeshStreamIn::FIntermediateRayTracingGeometry::CreateFromCPUData(FRHICommandListBase& RHICmdList, FRayTracingGeometry& RayTracingGeometry)
{
	Initializer = RayTracingGeometry.GetInitializer();
	Initializer.Type = ERayTracingGeometryInitializerType::StreamingSource;

	if (RayTracingGeometry.RawData.Num())
	{
		check(!RayTracing::ShouldForceRuntimeBLAS());
		check(Initializer.OfflineData == nullptr);
		Initializer.OfflineData = &RayTracingGeometry.RawData;
	}

	RayTracingGeometryRHI = RHICmdList.CreateRayTracingGeometry(Initializer);
	bRequiresBuild = Initializer.OfflineData == nullptr || RayTracingGeometryRHI->IsCompressed();
}

void FStaticMeshStreamIn::FIntermediateRayTracingGeometry::SafeRelease()
{
	Initializer = {};
	RayTracingGeometryRHI.SafeRelease();
}

void FStaticMeshStreamIn::FIntermediateRayTracingGeometry::TransferRayTracingGeometry(FRayTracingGeometry& RayTracingGeometry, FRHIResourceReplaceBatcher& Batcher)
{
	if (ensureMsgf(RayTracingGeometryRHI.IsValid(),
		TEXT("FIntermediateRayTracingGeometry should have a valid RHI object. Was r.RayTracing.Enable toggled between FStaticMeshStreamIn::CreateBuffers(...) and FStaticMeshStreamIn::DoFinishUpdate(...)?")))
	{
		RayTracingGeometry.InitRHIForStreaming(RayTracingGeometryRHI, Batcher);
		RayTracingGeometry.SetRequiresBuild(bRequiresBuild);
	}
}

#if DO_CHECK
static void CheckRayTracingGeometryInitializer(
	const FStaticMeshLODResources& LODResource,
	ERayTracingGeometryInitializerType ExpectedInitializerType,
	const FRayTracingGeometryInitializer& Initializer)
{
	FRayTracingGeometryInitializer TmpInitializer;
	LODResource.SetupRayTracingGeometryInitializer(TmpInitializer, Initializer.DebugName, Initializer.OwnerName);
	// simply copied names since they're not always available during streaming

	TmpInitializer.Type = ExpectedInitializerType;

	check(TmpInitializer == Initializer);
}
#endif

#endif

FStaticMeshStreamIn::FStaticMeshStreamIn(const UStaticMesh* InMesh, EThreadType CreateResourcesThread)
	: FStaticMeshUpdate(InMesh)
	, CreateResourcesThread(CreateResourcesThread)
{}

FStaticMeshStreamIn::~FStaticMeshStreamIn()
{
	check(!StreamingRHICmdList);
}

void FStaticMeshStreamIn::CreateBuffers(const FContext& Context)
{
	LLM_SCOPE(ELLMTag::StaticMesh);

	check(Context.Mesh && Context.RenderData);

	StreamingRHICmdList = new FRHICommandList();
	StreamingRHICmdList->SwitchPipeline(ERHIPipeline::Graphics);
	{
		SCOPED_DRAW_EVENTF(*StreamingRHICmdList, StaticMesh_StreamIn, TEXT("StaticMesh - StreamIn: %s"), Context.Mesh->GetFName());

		for (int32 LODIdx = PendingFirstLODIdx; LODIdx < CurrentFirstLODIdx; ++LODIdx)
		{
			FStaticMeshLODResources& LODResource = *Context.LODResourcesView[LODIdx];

			IntermediateBuffersArray[LODIdx].CreateFromCPUData(*StreamingRHICmdList, LODResource);

#if RHI_RAYTRACING
			if (IsRayTracingEnabled() && LODResource.RayTracingGeometry != nullptr && LODResource.GetNumVertices() > 0)
			{
#if DO_CHECK
				CheckRayTracingGeometryInitializer(LODResource, ERayTracingGeometryInitializerType::StreamingDestination, LODResource.RayTracingGeometry->GetInitializer());
#endif

				IntermediateRayTracingGeometry[LODIdx].CreateFromCPUData(*StreamingRHICmdList, *LODResource.RayTracingGeometry);
			}
#endif
		}

		// Use a scope to flush the batcher before updating CurrentFirstLODIdx
		{
			FRHIResourceReplaceBatcher Batcher(*StreamingRHICmdList, GStaticMeshMaxNumResourceUpdatesPerBatch);
			for (int32 LODIdx = PendingFirstLODIdx; LODIdx < CurrentFirstLODIdx; ++LODIdx)
			{
				FStaticMeshLODResources& LODResource = *Context.LODResourcesView[LODIdx];
				LODResource.IncrementMemoryStats();
				IntermediateBuffersArray[LODIdx].TransferBuffers(LODResource, Batcher);
			}
		}
	}

	StreamingRHICmdList->FinishRecording();
}

void FStaticMeshStreamIn::DiscardNewLODs(const FContext& Context)
{
	if (Context.RenderData)
	{
		for (int32 LODIdx = PendingFirstLODIdx; LODIdx < CurrentFirstLODIdx; ++LODIdx)
		{
			FStaticMeshLODResources& LODResource = *Context.LODResourcesView[LODIdx];
			LODResource.DiscardCPUData();
		}
	}
}

void FStaticMeshStreamIn::DoFinishUpdate(const FContext& Context)
{
	check(Context.CurrentThread == TT_Render);
	check(IsInRenderingThread());
	
	if (StreamingRHICmdList)
	{
		FRHICommandListImmediate::Get().QueueAsyncCommandListSubmit(StreamingRHICmdList);
		StreamingRHICmdList = nullptr;
	}

#if RHI_RAYTRACING
	if (IsRayTracingAllowed() && Context.Mesh->bSupportRayTracing)
	{
		// Use a scope to flush the batcher before updating CurrentFirstLODIdx
		{
			FRHIResourceReplaceBatcher Batcher(FRHICommandListImmediate::Get(), GStaticMeshMaxNumResourceUpdatesPerBatch);
			for (int32 LODIdx = PendingFirstLODIdx; LODIdx < CurrentFirstLODIdx; ++LODIdx)
			{
				FStaticMeshLODResources& LODResource = *Context.LODResourcesView[LODIdx];
				if (IsRayTracingEnabled() && LODResource.GetNumVertices() > 0 && LODResource.RayTracingGeometry != nullptr && !LODResource.RayTracingGeometry->IsEvicted())
				{
					IntermediateRayTracingGeometry[LODIdx].TransferRayTracingGeometry(*LODResource.RayTracingGeometry, Batcher);
				}

				IntermediateRayTracingGeometry[LODIdx].SafeRelease();
			}
		}

		// Must happen after the batched updates have been flushed
		for (int32 LODIndex = PendingFirstLODIdx; LODIndex < CurrentFirstLODIdx; ++LODIndex)
		{
			FStaticMeshLODResources& LODResource = *Context.LODResourcesView[LODIndex];

			// Skip LODs that have their render data stripped or are evicted
			if (LODResource.GetNumVertices() > 0 && LODResource.RayTracingGeometry != nullptr && !LODResource.RayTracingGeometry->IsEvicted())
			{
#if DO_CHECK
				// Streaming LODs in/out shouldn't affect the ray tracing geometry initializer
				// Here we check that assumption
				CheckRayTracingGeometryInitializer(LODResource, ERayTracingGeometryInitializerType::Rendering, LODResource.RayTracingGeometry->GetInitializer());

				check(EnumHasAllFlags(LODResource.RayTracingGeometry->GetGeometryState(), FRayTracingGeometry::EGeometryStateFlags::StreamedIn));
#endif

				// Under very rare circumstances that we switch ray tracing on/off right in the middle of streaming RayTracingGeometryRHI might not be valid.
				if (IsRayTracingEnabled() && ensure(LODResource.RayTracingGeometry->IsValid()))
				{
					LODResource.RayTracingGeometry->RequestBuildIfNeeded(FRHICommandListImmediate::Get(), ERTAccelerationStructureBuildPriority::Normal);
				}
			}
		}

	}
#endif

	Context.RenderData->CurrentFirstLODIdx = ResourceState.LODCountToAssetFirstLODIdx(ResourceState.NumRequestedLODs);
	Context.Mesh->RequestUpdateCachedRenderState();
	MarkAsSuccessfullyFinished();
}

void FStaticMeshStreamIn::DoCancel(const FContext& Context)
{
	// TODO: support streaming CPU data for editor builds
	if (!FPlatformProperties::HasEditorOnlyData())
	{
		DiscardNewLODs(Context);
	}

	check(!StreamingRHICmdList);
}

FStaticMeshStreamOut::FStaticMeshStreamOut(const UStaticMesh* InMesh, bool InDiscardCPUData)
	: FStaticMeshUpdate(InMesh)
	, bDiscardCPUData(InDiscardCPUData)
{
	check(InMesh);

	// Immediately change CurrentFirstLODIdx to prevent new references from being made to the streamed out lods.
	// #TODO RenderData under a const UStaticMesh should stay const
	FStaticMeshRenderData* RenderData = const_cast<UStaticMesh*>(InMesh)->GetRenderData();
	if (RenderData)
	{
		RenderData->CurrentFirstLODIdx = ResourceState.LODCountToAssetFirstLODIdx(ResourceState.NumRequestedLODs);
	}

	if (InDiscardCPUData)
	{
		PushTask(FContext(InMesh, TT_None), TT_Async, SRA_UPDATE_CALLBACK(CheckReferencesAndDiscardCPUData), TT_Async, SRA_UPDATE_CALLBACK(Cancel));
	}
	else
	{
		PushTask(FContext(InMesh, TT_None), TT_Render, SRA_UPDATE_CALLBACK(ReleaseRHIBuffers), TT_Async, SRA_UPDATE_CALLBACK(Cancel));
	}
}

void FStaticMeshStreamOut::CheckReferencesAndDiscardCPUData(const FContext& Context)
{
	DECLARE_SCOPE_CYCLE_COUNTER(TEXT("FStaticMeshStreamOut::CheckReferencesAndDiscardCPUData"), STAT_StaticMeshStreamOut_CheckReferencesAndDiscardCPUData, STATGROUP_StreamingDetails);
	check(Context.CurrentThread == TT_Async);

	const UStaticMesh* Mesh = Context.Mesh;
	FStaticMeshRenderData* RenderData = Context.RenderData;
	uint32 NumExternalReferences = 0;

	if (Mesh && RenderData)
	{
		for (int32 LODIdx = CurrentFirstLODIdx; LODIdx < PendingFirstLODIdx; ++LODIdx)
		{
			// Minus 1 since the LODResources reference is not considered external
			NumExternalReferences += Context.LODResourcesView[LODIdx]->GetRefCount() - 1;
		}

		if (NumExternalReferences > PreviousNumberOfExternalReferences && NumReferenceChecks > 0)
		{
			PreviousNumberOfExternalReferences = NumExternalReferences;
			UE_LOGF(LogContentStreaming, Warning, "[%ls] Streamed out LODResources got referenced while in pending stream out.", *Mesh->GetName());
		}
	}

	if (!NumExternalReferences || NumReferenceChecks >= GStreamingMaxReferenceChecks)
	{
		if (RenderData)
		{
			for (int32 LODIdx = CurrentFirstLODIdx; LODIdx < PendingFirstLODIdx; ++LODIdx)
			{
				Context.LODResourcesView[LODIdx]->DiscardCPUData();
			}
		}

		// Because we discarded the CPU data, the stream out can not be cancelled anymore.
		PushTask(Context, TT_Render, SRA_UPDATE_CALLBACK(ReleaseRHIBuffers), TT_Render, SRA_UPDATE_CALLBACK(ReleaseRHIBuffers));
	}
	else
	{
		++NumReferenceChecks;
		if (NumReferenceChecks >= GStreamingMaxReferenceChecks)
		{
			UE_LOGF(LogContentStreaming, Warning, "[%ls] Streamed out LODResources references are not getting released.", *Mesh->GetName());
		}

		bDeferExecution = true;
		PushTask(Context, TT_Async, SRA_UPDATE_CALLBACK(CheckReferencesAndDiscardCPUData), TT_Async, SRA_UPDATE_CALLBACK(Cancel));
	}
}

void FStaticMeshStreamOut::ReleaseRHIBuffers(const FContext& Context)
{
	DECLARE_SCOPE_CYCLE_COUNTER(TEXT("FStaticMeshStreamOut::ReleaseRHIBuffers"), STAT_StaticMeshStreamOut_ReleaseRHIBuffers, STATGROUP_StreamingDetails);
	check(Context.CurrentThread == TT_Render);

	if (Context.RenderData)
	{
		FRHIResourceReplaceBatcher Batcher(FRHICommandListImmediate::Get(), GStaticMeshMaxNumResourceUpdatesPerBatch);
		for (int32 LODIdx = CurrentFirstLODIdx; LODIdx < PendingFirstLODIdx; ++LODIdx)
		{
			FStaticMeshLODResources& LODResource = *Context.LODResourcesView[LODIdx];
			LODResource.DecrementMemoryStats();
			LODResource.ReleaseRHIForStreaming(Batcher);
			
#if RHI_RAYTRACING
			if (LODResource.RayTracingGeometry != nullptr && !LODResource.RayTracingGeometry->IsEvicted())
			{
				check(IsRayTracingAllowed());
				LODResource.RayTracingGeometry->ReleaseRHIForStreaming(Batcher);
			}
#endif
		}

		Context.Mesh->RequestUpdateCachedRenderState();
	}
	MarkAsSuccessfullyFinished();
}

void FStaticMeshStreamOut::Cancel(const FContext& Context)
{
	DECLARE_SCOPE_CYCLE_COUNTER(TEXT("FStaticMeshStreamOut::Cancel"), STAT_StaticMeshStreamOut_Cancel, STATGROUP_StreamingDetails);
	check(Context.CurrentThread == TT_Async);

	if (Context.RenderData)
	{
		Context.RenderData->CurrentFirstLODIdx = ResourceState.LODCountToAssetFirstLODIdx(ResourceState.NumResidentLODs);
	}
}

void FStaticMeshStreamIn_IO::FCancelIORequestsTask::DoWork()
{
	check(PendingUpdate);
	// Acquire the lock of this object in order to cancel any pending IO.
	// If the object is currently being ticked, wait.
	const ETaskState PreviousTaskState = PendingUpdate->DoLock();
	PendingUpdate->CancelIORequest();
	PendingUpdate->DoUnlock(PreviousTaskState);
}

void FStaticMeshStreamIn_IO::Abort()
{
	if (!IsCancelled() && !IsCompleted())
	{
		FStaticMeshStreamIn::Abort();

		if (BulkDataRequest.IsPending())
		{
			// Prevent the update from being considered done before this is finished.
			// By checking that it was not already cancelled, we make sure this doesn't get called twice.
			(new FAsyncCancelIORequestsTask(this))->StartBackgroundTask();
		}
	}
}

void FStaticMeshStreamIn_IO::SetIORequest(const FContext& Context)
{
	if (IsCancelled())
	{
		return;
	}

	check(BulkDataRequest.IsNone() && PendingFirstLODIdx < CurrentFirstLODIdx);

	if (Context.Mesh && Context.RenderData)
	{
		TRACE_IOSTORE_METADATA_SCOPE_TAG(*Context.Mesh->GetName());

		const int32 BatchCount = CurrentFirstLODIdx - PendingFirstLODIdx;
		FBulkDataBatchRequest::FScatterGatherBuilder Batch = FBulkDataBatchRequest::ScatterGather(BatchCount);
		for (int32 Index = PendingFirstLODIdx; Index < CurrentFirstLODIdx; ++Index)
		{
			Batch.Read(Context.LODResourcesView[Index]->StreamingBulkData);
		}
		
		// Increment as we push the request. If a request complete immediately, then it will call the callback
		// but that won't do anything because the tick would not try to acquire the lock since it is already locked.
		TaskSynchronization.Increment();

		EAsyncIOPriorityAndFlags Priority = AIOP_Low;
		if (bHighPrioIORequest)
		{
			static IConsoleVariable* CVarAsyncLoadingPrecachePriority = IConsoleManager::Get().FindConsoleVariable(TEXT("s.AsyncLoadingPrecachePriority"));
			const bool bLoadBeforeAsyncPrecache = CVarStreamingLowResHandlingMode.GetValueOnAnyThread() == (int32)FRenderAssetStreamingSettings::LRHM_LoadBeforeAsyncPrecache;

			if (CVarAsyncLoadingPrecachePriority && bLoadBeforeAsyncPrecache)
			{
				const int32 AsyncIOPriority = CVarAsyncLoadingPrecachePriority->GetInt();
				// Higher priority than regular requests but don't go over max
				Priority = (EAsyncIOPriorityAndFlags)FMath::Clamp<int32>(AsyncIOPriority + 1, AIOP_BelowNormal, AIOP_MAX);
			}
			else
			{
				Priority = AIOP_BelowNormal;
			}
		}

		Batch.Issue(BulkData, Priority, [this](FBulkDataRequest::EStatus Status)
		{
			TaskSynchronization.Decrement();

			if (FBulkDataRequest::EStatus::Ok != Status)
			{
				// If IO requests was cancelled but the streaming request wasn't, this is an IO error.
				if (!bIsCancelled)
				{
					bFailedOnIOError = true;
				}
				MarkAsCancelled();
			}

#if !UE_BUILD_SHIPPING
			// On some platforms the IO is too fast to test cancelation requests timing issues.
			if (FRenderAssetStreamingSettings::ExtraIOLatency > 0 && TaskSynchronization.GetValue() == 0)
			{
				FPlatformProcess::Sleep(FRenderAssetStreamingSettings::ExtraIOLatency * .001f); // Slow down the streaming.
			}
#endif
			// The tick here is intended to schedule the success or cancel callback.
			// Using TT_None ensure gets which could create a dead lock.
			Tick(FStaticMeshUpdate::TT_None);
		},
		BulkDataRequest);
	}
	else
	{
		MarkAsCancelled();
	}
} 

void FStaticMeshStreamIn_IO::ClearIORequest(const FContext& Context)
{
	if (BulkDataRequest.IsPending())
	{
		BulkDataRequest.Cancel();
		BulkDataRequest.Wait();
	}
	
	BulkDataRequest = FBulkDataBatchRequest();
	BulkData = FIoBuffer();
}

void FStaticMeshStreamIn_IO::ReportIOError(const FContext& Context)
{
	// Invalidate the cache state of all initial mips (note that when using FIoChunkId each mip has a different value).
	if (bFailedOnIOError && Context.Mesh)
	{
		IRenderAssetStreamingManager& StreamingManager = IStreamingManager::Get().GetTextureStreamingManager();
		for (int32 MipIndex = 0; MipIndex < CurrentFirstLODIdx; ++MipIndex)
		{
			StreamingManager.MarkMountedStateDirty(Context.Mesh->GetMipIoFilenameHash(MipIndex));
		}

		UE_LOGF(LogContentStreaming, Warning, "[%ls] StaticMesh stream in request failed due to IO error (LOD %d-%d).", *Context.Mesh->GetName(), PendingFirstLODIdx, CurrentFirstLODIdx - 1);
	}
}

void FStaticMeshStreamIn_IO::SerializeLODData(const FContext& Context)
{
	LLM_SCOPE(ELLMTag::StaticMesh);

	check(!TaskSynchronization.GetValue());
	const UStaticMesh* Mesh = Context.Mesh;
	FStaticMeshRenderData* RenderData = Context.RenderData;
	if (!IsCancelled() && Mesh && RenderData)
	{
		check(BulkData.GetSize() >= 0 && BulkData.GetSize() <= TNumericLimits<uint32>::Max());

		FMemoryReaderView Ar(BulkData.GetView(), true);
		for (int32 LODIdx = PendingFirstLODIdx; LODIdx < CurrentFirstLODIdx; ++LODIdx)
		{
			FStaticMeshLODResources& LODResource = *Context.LODResourcesView[LODIdx];
			constexpr uint8 DummyStripFlags = 0;
			typename FStaticMeshLODResources::FStaticMeshBuffersSize DummyBuffersSize;
			LODResource.SerializeBuffers(Ar, const_cast<UStaticMesh*>(Mesh), DummyStripFlags, DummyBuffersSize);

			// Attempt to recover from possibly corrupted data if allowed
			if (Ar.IsError())
			{
				UE_LOGF(LogContentStreaming, Error,
					"[%ls] StaticMesh stream in failed due to possibly corrupted data. LOD %d %d-%d. BulkData %#x offset %lld size %lld flags %#x.",
					*Mesh->GetPathName(),
					LODIdx,
					PendingFirstLODIdx,
					CurrentFirstLODIdx - 1,
					LODResource.StreamingBulkData.GetIoFilenameHash(),
					LODResource.StreamingBulkData.GetBulkDataOffsetInFile(),
					LODResource.StreamingBulkData.GetBulkDataSize(),
					LODResource.StreamingBulkData.GetBulkDataFlags());

#if STREAMING_RETRY_ON_DESERIALIZATION_ERROR
				bFailedOnIOError = true;
				MarkAsCancelled();
				break;
#else
				GLog->FlushThreadedLogs();
				GLog->Flush();
				UE_LOGF(LogContentStreaming, Fatal, "Possibly corrupted static mesh LOD data detected.");
#endif
			}
		}

		BulkData = FIoBuffer();
	}
}

void FStaticMeshStreamIn_IO::Cancel(const FContext& Context)
{
	DoCancel(Context);
	ReportIOError(Context);
}

void FStaticMeshStreamIn_IO::CancelIORequest()
{
	if (BulkDataRequest.IsPending())
	{
		BulkDataRequest.Cancel();
	}
}

FStaticMeshStreamIn_IO::FStaticMeshStreamIn_IO(const UStaticMesh* InMesh, bool bHighPrio, EThreadType CreateResourcesThread)
	: FStaticMeshStreamIn(InMesh, CreateResourcesThread)
	, bHighPrioIORequest(bHighPrio)
{
	PushTask(FContext(InMesh, TT_None), TT_Async, SRA_UPDATE_CALLBACK(DoInitiateIO), TT_None, nullptr);
}

void FStaticMeshStreamIn_IO::DoInitiateIO(const FContext& Context)
{
	DECLARE_SCOPE_CYCLE_COUNTER(TEXT("FStaticMeshStreamIn_IO::DoInitiateIO"), STAT_StaticMeshStreamInIO_DoInitiateIO, STATGROUP_StreamingDetails);
	check(Context.CurrentThread == TT_Async);

	SetIORequest(Context);

	PushTask(Context, TT_Async, SRA_UPDATE_CALLBACK(DoSerializeLODData), TT_Async, SRA_UPDATE_CALLBACK(DoCancelIO));
}

void FStaticMeshStreamIn_IO::DoSerializeLODData(const FContext& Context)
{
	DECLARE_SCOPE_CYCLE_COUNTER(TEXT("FStaticMeshStreamIn_IO::DoSerializeLODData"), STAT_StaticMeshStreamInIO_DoSerializeLODData, STATGROUP_StreamingDetails);
	check(Context.CurrentThread == TT_Async);

	SerializeLODData(Context);
	ClearIORequest(Context);

	PushTask(Context
		, CreateResourcesThread, SRA_UPDATE_CALLBACK(DoCreateBuffers)
		, (EThreadType)Context.CurrentThread, SRA_UPDATE_CALLBACK(Cancel));
}

void FStaticMeshStreamIn_IO::DoCreateBuffers(const FContext& Context)
{
	DECLARE_SCOPE_CYCLE_COUNTER(TEXT("FStaticMeshStreamIn_IO::DoCreateBuffers"), STAT_StaticMeshStreamInIO_DoCreateBuffers, STATGROUP_StreamingDetails);
	CreateBuffers(Context);

	check(!TaskSynchronization.GetValue());

	// We cannot cancel once DoCreateBuffers has started executing, as there's an RHICmdList that must be submitted.
	// Pass the same callback for both task and cancel.
	PushTask(Context
		, TT_Render, SRA_UPDATE_CALLBACK(DoFinishUpdate)
		, TT_Render, SRA_UPDATE_CALLBACK(DoFinishUpdate)
	);
}

void FStaticMeshStreamIn_IO::DoCancelIO(const FContext& Context)
{
	DECLARE_SCOPE_CYCLE_COUNTER(TEXT("FStaticMeshStreamIn_IO::DoCancelIO"), STAT_StaticMeshStreamInIO_DoCancelIO, STATGROUP_StreamingDetails);
	ClearIORequest(Context);

	PushTask(Context, TT_None, nullptr, (EThreadType)Context.CurrentThread, SRA_UPDATE_CALLBACK(Cancel));
}

#if WITH_EDITOR
FStaticMeshStreamIn_DDC::FStaticMeshStreamIn_DDC(const UStaticMesh* InMesh, EThreadType CreateResourcesThread)
	: FStaticMeshStreamIn(InMesh, CreateResourcesThread)
{
	PushTask(FContext(InMesh, TT_None), TT_Async, SRA_UPDATE_CALLBACK(DoLoadNewLODsFromDDC), TT_None, nullptr);
}

void FStaticMeshStreamIn_DDC::LoadNewLODsFromDDC(const FContext& Context)
{
	check(Context.CurrentThread == TT_Async);
	// TODO: support streaming CPU data for editor builds
}

void FStaticMeshStreamIn_DDC::DoLoadNewLODsFromDDC(const FContext& Context)
{
	DECLARE_SCOPE_CYCLE_COUNTER(TEXT("FStaticMeshStreamIn_DDC::DoLoadNewLODsFromDDC"), STAT_StaticMeshStreamInDDC_DoLoadNewLODsFromDDC, STATGROUP_StreamingDetails);
	LoadNewLODsFromDDC(Context);
	check(!TaskSynchronization.GetValue());

	PushTask(Context
		, CreateResourcesThread, SRA_UPDATE_CALLBACK(DoCreateBuffers)
		, (EThreadType)Context.CurrentThread, SRA_UPDATE_CALLBACK(DoCancel));
}

void FStaticMeshStreamIn_DDC::DoCreateBuffers(const FContext& Context)
{
	DECLARE_SCOPE_CYCLE_COUNTER(TEXT("FStaticMeshStreamIn_DDC::DoCreateBuffers"), STAT_StaticMeshStreamInDDC_DoCreateBuffers, STATGROUP_StreamingDetails);
	CreateBuffers(Context);

	check(!TaskSynchronization.GetValue());
	PushTask(Context, TT_Render, SRA_UPDATE_CALLBACK(DoFinishUpdate), TT_None, nullptr);
}
#endif
