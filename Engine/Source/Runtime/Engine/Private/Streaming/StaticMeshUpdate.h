// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
StaticMeshUpdate.h: Helpers to stream in and out static mesh LODs.
=============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "Engine/StaticMesh.h"
#include "Async/AsyncFileHandle.h"
#include "IO/IoDispatcher.h"
#include "RenderAssetUpdate.h"
#include "RayTracingGeometry.h"

/**
* A context used to update or proceed with the next update step.
* The mesh and render data references could be stored in the update object
* but are currently kept outside to avoid lifetime management within the object.
*/
struct FStaticMeshUpdateContext
{
	typedef int32 EThreadType;

	FStaticMeshUpdateContext(const UStaticMesh* InMesh, EThreadType InCurrentThread);

	FStaticMeshUpdateContext(const UStreamableRenderAsset* InMesh, EThreadType InCurrentThread);

	EThreadType GetCurrentThread() const
	{
		return CurrentThread;
	}

	/** The mesh to update, this must be the same one as the one used when creating the FStaticMeshUpdate object. */
	const UStaticMesh* Mesh;
	/** The current render data of this mesh. */
	FStaticMeshRenderData* RenderData;
	/** The array view of streamable LODs from the asset. Takes into account FStreamableRenderResourceState::AssetLODBias and FStreamableRenderResourceState::MaxNumLODs. */
	TArrayView<FStaticMeshLODResources*> LODResourcesView;

	/** The thread on which the context was created. */
	EThreadType CurrentThread;
};

// Declare that TRenderAssetUpdate is instantiated for FStaticMeshUpdateContext
extern template class TRenderAssetUpdate<FStaticMeshUpdateContext>;

/**
* This class provides a framework for loading and unloading the LODs of static meshes.
* Each thread essentially calls Tick() until the job is done.
* The object can be safely deleted when IsCompleted() returns true.
*/
class FStaticMeshUpdate : public TRenderAssetUpdate<FStaticMeshUpdateContext>
{
public:
	FStaticMeshUpdate(const UStaticMesh* InMesh);

	virtual void Abort()
	{
		TRenderAssetUpdate<FStaticMeshUpdateContext>::Abort();
	}

protected:

	virtual ~FStaticMeshUpdate() = default;
};

class FStaticMeshStreamIn : public FStaticMeshUpdate
{
public:
	FStaticMeshStreamIn(const UStaticMesh* InMesh, EThreadType CreateResourcesThread);

	virtual ~FStaticMeshStreamIn();

protected:
	/** Correspond to the buffers in FStaticMeshLODResources */
	struct FIntermediateBuffers
	{
		FBufferRHIRef TangentsVertexBuffer;
		FBufferRHIRef TexCoordVertexBuffer;
		FBufferRHIRef PositionVertexBuffer;
		FBufferRHIRef ColorVertexBuffer;
		FBufferRHIRef IndexBuffer;
		FBufferRHIRef ReversedIndexBuffer;
		FBufferRHIRef DepthOnlyIndexBuffer;
		FBufferRHIRef ReversedDepthOnlyIndexBuffer;
		FBufferRHIRef WireframeIndexBuffer;

		void CreateFromCPUData(FRHICommandListBase& RHICmdList, FStaticMeshLODResources& LODResource);

		/** Transfer ownership of buffers to a LOD resource */
		void TransferBuffers(FStaticMeshLODResources& LODResource, FRHIResourceReplaceBatcher& Batcher);
	};

#if RHI_RAYTRACING
	struct FIntermediateRayTracingGeometry
	{
	private:
		FRayTracingGeometryInitializer Initializer;
		FRayTracingGeometryRHIRef RayTracingGeometryRHI;
		bool bRequiresBuild = false;

	public:
		void CreateFromCPUData(FRHICommandListBase& RHICmdList, FRayTracingGeometry& RayTracingGeometry);

		void SafeRelease();

		void TransferRayTracingGeometry(FRayTracingGeometry& RayTracingGeometry, FRHIResourceReplaceBatcher& Batcher);
	};
#endif

	/** Create buffers with new LOD data */
	void CreateBuffers(const FContext& Context);

	/** Discard newly streamed-in CPU data */
	void DiscardNewLODs(const FContext& Context);

	/** Apply the new buffers (if not cancelled) and finish the update process. When cancelled, the intermediate buffers simply gets discarded. */
	void DoFinishUpdate(const FContext& Context);

	/** Discard streamed-in CPU data and intermediate RHI buffers */
	void DoCancel(const FContext& Context);

	/** The intermediate buffers created in the update process. */
	FIntermediateBuffers IntermediateBuffersArray[MAX_MESH_LOD_COUNT];
	
#if RHI_RAYTRACING
	FIntermediateRayTracingGeometry IntermediateRayTracingGeometry[MAX_MESH_LOD_COUNT];
#endif

	/** RHI command list used for creating buffers and replacing the streaming placeholders. Submitted in DoFinishUpdate */
	FRHICommandList* StreamingRHICmdList = nullptr;

	/** The thread to use for recording the above command list */
	const EThreadType CreateResourcesThread;
};

/** A streamout that doesn't actually touches the CPU data. Required because DDC stream in doesn't reset. */
class FStaticMeshStreamOut : public FStaticMeshUpdate
{
public:
	FStaticMeshStreamOut(const UStaticMesh* InMesh, bool InDiscardCPUData);

private:

	void CheckReferencesAndDiscardCPUData(const FContext& Context);
	void ReleaseRHIBuffers(const FContext& Context);
	/** Restore */
	void Cancel(const FContext& Context);

	bool bDiscardCPUData = false;
	int32 NumReferenceChecks = 0;
	uint32 PreviousNumberOfExternalReferences = 0;
};

class FStaticMeshStreamIn_IO : public FStaticMeshStreamIn
{
public:
	FStaticMeshStreamIn_IO(const UStaticMesh* InMesh, bool bHighPrio, EThreadType CreateResourcesThread);

	virtual ~FStaticMeshStreamIn_IO() = default;

	virtual void Abort() override;

protected:
	class FCancelIORequestsTask : public FNonAbandonableTask
	{
	public:
		FCancelIORequestsTask(FStaticMeshStreamIn_IO* InPendingUpdate)
			: PendingUpdate(InPendingUpdate)
		{}

		void DoWork();

		FORCEINLINE TStatId GetStatId() const
		{
			RETURN_QUICK_DECLARE_CYCLE_STAT(FCancelIORequestsTask_StaticMesh, STATGROUP_ThreadPoolAsyncTasks);
		}

	private:
		TRefCountPtr<FStaticMeshStreamIn_IO> PendingUpdate;
	};

	typedef FAutoDeleteAsyncTask<FCancelIORequestsTask> FAsyncCancelIORequestsTask;
	friend class FCancelIORequestsTask;

	/** Create a new async IO request to read in LOD data */
	void SetIORequest(const FContext& Context);

	/** Release IORequest and IOFileHandle. IORequest will be cancelled if still inflight */
	void ClearIORequest(const FContext& Context);

	/** Report IO errors if any. */
	void ReportIOError(const FContext& Context);

	/** Serialize data of new LODs to corresponding FStaticMeshLODResources */
	void SerializeLODData(const FContext& Context);

	/** Cancel and report IO error */
	void Cancel(const FContext& Context);

	/** Called by FAsyncCancelIORequestsTask to cancel inflight IO request if any */
	void CancelIORequest();

	/** Handle to bulk data request */;
	FBulkDataBatchRequest BulkDataRequest;

	/** Bulk data I/O buffer */
	FIoBuffer BulkData;

	bool bHighPrioIORequest;

	// Whether an IO error was detected (when files do not exists).
	bool bFailedOnIOError = false;

	void DoInitiateIO(const FContext& Context);

	void DoSerializeLODData(const FContext& Context);

	void DoCreateBuffers(const FContext& Context);

	void DoCancelIO(const FContext& Context);
};

#if WITH_EDITOR
class FStaticMeshStreamIn_DDC : public FStaticMeshStreamIn
{
public:
	FStaticMeshStreamIn_DDC(const UStaticMesh* InMesh, EThreadType CreateResourcesThread);

	virtual ~FStaticMeshStreamIn_DDC() = default;

protected:
	void LoadNewLODsFromDDC(const FContext& Context);

private:
	/** Load new LOD buffers from DDC and queue a task to create RHI buffers on RT */
	void DoLoadNewLODsFromDDC(const FContext& Context);

	/** Create RHI buffers for newly streamed-in LODs and queue a task to rename references on RT */
	void DoCreateBuffers(const FContext& Context);
};
#endif
