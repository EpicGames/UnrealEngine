// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	StaticTextureInstanceManager.cpp: Implementation of content streaming classes.
=============================================================================*/

#include "Streaming/StaticTextureInstanceManager.h"
#include "Components/PrimitiveComponent.h"

void FStaticRenderAssetInstanceManager::FTasks::SyncResults()
{
	RefreshVisibilityTask->TryWork(false);
	NormalizeLightmapTexelFactorTask->TryWork(false);
	// All (async) work must be completed before synching the results as the work assume a constant state.
	RefreshVisibilityTask->TrySync();
	NormalizeLightmapTexelFactorTask->TrySync();

}

FStaticRenderAssetInstanceManager::FStaticRenderAssetInstanceManager(RenderAssetInstanceTask::FDoWorkTask& AsyncTask)
	: DirtyIndex(0)
{
	FTasks& Tasks = StateSync.GetTasks();

	Tasks.RefreshVisibilityTask = new FRefreshVisibilityTask(RenderAssetInstanceTask::FRefreshVisibility::FOnWorkDone::CreateLambda([this](int32 InBeginIndex, int32 InEndIndex){ this->OnRefreshVisibilityDone(InBeginIndex, InEndIndex); }));
	AsyncTask.Add(Tasks.RefreshVisibilityTask.GetReference());

	Tasks.NormalizeLightmapTexelFactorTask = new FNormalizeLightmapTexelFactorTask();
	AsyncTask.Add(Tasks.NormalizeLightmapTexelFactorTask.GetReference());

}

void FStaticRenderAssetInstanceManager::NormalizeLightmapTexelFactor()
{
	if (!AsyncView)
	{
		FRenderAssetInstanceState* State = StateSync.SyncAndGetState();
		if (State->NumBounds() > 0)
		{
			StateSync.GetTasks().NormalizeLightmapTexelFactorTask->Init(State);
		}
	}
}

void FStaticRenderAssetInstanceManager::OnRefreshVisibilityDone(int32 InBeginIndex, int32 InEndIndex)
{
	// Make sure there are no wholes between the DirtyIndex and the first updated index.
	if (InBeginIndex <= DirtyIndex)
	{
		DirtyIndex = FMath::Max<int32>(InEndIndex, DirtyIndex);
	}
}

void FStaticRenderAssetInstanceManager::Refresh(float Percentage)
{
	// Since this is only managing static components, only visibility needs to be refreshed.

	FRenderAssetInstanceState* State = StateSync.SyncAndGetState();
	if (DirtyIndex < State->NumBounds())
	{
		const int32 EndIndex = FMath::Min(State->NumBounds(), DirtyIndex + FMath::CeilToInt((float)State->NumBounds() * Percentage));
		StateSync.GetTasks().RefreshVisibilityTask->Init(State, DirtyIndex, EndIndex);
	}
}

bool FStaticRenderAssetInstanceManager::CanAddComponent() const
{
	return !AsyncView;
}

EAddComponentResult FStaticRenderAssetInstanceManager::Add(const TArray<FRenderAssetInstanceState::FPreAddComponentPayload>& Payloads, TFunctionRef<void(const FRenderAssetInstanceState::FPreAddComponentPayload&, EAddComponentResult)> OnAddFailedFunc)
{
	if (!AsyncView)
	{
		FRenderAssetInstanceState* State = StateSync.SyncAndGetState();
		return State->AddComponents(Payloads, OnAddFailedFunc);
	}
	return EAddComponentResult::Fail;
}

EAddComponentResult FStaticRenderAssetInstanceManager::Add(const IPrimitiveComponent* Component, FStreamingTextureLevelContext& LevelContext, float MaxAllowedUIDensity)
{
	if (!AsyncView)
	{
		FRenderAssetInstanceState* State = StateSync.SyncAndGetState();
		return State->AddComponent(Component, LevelContext, MaxAllowedUIDensity);
	}
	return EAddComponentResult::Fail;
}

void FStaticRenderAssetInstanceManager::Remove(const IPrimitiveComponent* Component, FRemovedRenderAssetArray* RemovedRenderAssets)
{
	FRenderAssetInstanceState* State = StateSync.SyncAndGetState();
	// If the view is shared, we are limited to clearing the references (no realloc)
	if (AsyncView)
	{
		State->RemoveComponentReferences(Component);
	}
	else // Otherwise it can be cleaned properly
	{
		State->RemoveComponent(Component, RemovedRenderAssets);
	}
}

const FRenderAssetInstanceView* FStaticRenderAssetInstanceManager::GetAsyncView(bool bCreateIfNull)
{
	FRenderAssetInstanceState* State = StateSync.SyncAndGetState();
	if (!AsyncView && bCreateIfNull)
	{
		AsyncView = State;
	}
	DirtyIndex = 0; // Force a full refresh!
	return AsyncView.GetReference();
}

uint32 FStaticRenderAssetInstanceManager::GetAllocatedSize() const
{
	const FRenderAssetInstanceState* State = StateSync.GetState();
	return  State ? (sizeof(FRenderAssetInstanceState) + State->GetAllocatedSize()) : 0;
}


void FStaticRenderAssetInstanceManager::OffsetBounds(const FVector& Offset)
{
	FRenderAssetInstanceState* State = StateSync.SyncAndGetState();
	if (State)
	{
		State->OffsetBounds(Offset);
	}
}