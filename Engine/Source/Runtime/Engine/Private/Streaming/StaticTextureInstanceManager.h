// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Streaming/TextureInstanceTask.h"

/** 
 * A texture/mesh instance manager to manage fully static components. 
 * Once an async view has been requested, nothing can be changed as the async view and the internal state point to the same data. 
 * This allows for quicker refreshes and also prevent state duplication.
 */
class FStaticRenderAssetInstanceManager
{
public:

	/** Contructor. */
	FStaticRenderAssetInstanceManager(RenderAssetInstanceTask::FDoWorkTask& AsyncTask);
	~FStaticRenderAssetInstanceManager() { StateSync.Sync(); }

	/** Normalize lightmap texel factors, this is ran on an async tasks. */
	void NormalizeLightmapTexelFactor();

	FORCEINLINE int32 CompileElements() { return StateSync.SyncAndGetState()->CompileElements(); }
	FORCEINLINE int32 CheckRegistrationAndUnpackBounds(TArray<const IPrimitiveComponent*>& RemovedComponents) { return StateSync.SyncAndGetState()->CheckRegistrationAndUnpackBounds(RemovedComponents); }
	FORCEINLINE FRenderAssetInstanceState::FRenderAssetIterator GetRenderAssetIterator( ) {  return StateSync.SyncAndGetState()->GetRenderAssetIterator(); }

	FORCEINLINE bool HasRenderAssetReferences() const { return StateSync.GetState()->NumBounds() > 0; }

	/** Return whether this component can be managed by this manager. */
	FORCEINLINE bool IsReferenced(const IPrimitiveComponent* Component) const { return StateSync.GetState()->HasComponentReferences(Component); }

	/** Invoke a callable for each referenced component. */
	template<typename CallableT>
	void ForEachReferencedComponent(CallableT&& Callable)
	{
		StateSync.GetState()->ForEachReferencedComponent(Forward<CallableT>(Callable));
	}

	/** Refresh component data (bounds, last render time, min and max view distance) - see RenderAssetInstanceView. */
	void Refresh(float Percentage);

	/** Add a component streaming data, the LevelContext gives support for precompiled data. */
	EAddComponentResult Add(const IPrimitiveComponent* Component, FStreamingTextureLevelContext& LevelContext, float MaxAllowedUIDensity);

	/** Add a batch of components streaming data. */
	EAddComponentResult Add(const TArray<FRenderAssetInstanceState::FPreAddComponentPayload>& Payloads, TFunctionRef<void(const FRenderAssetInstanceState::FPreAddComponentPayload&, EAddComponentResult)> OnAddFailedFunc);

	/** Remove a component, the RemovedRenderAssets is the list of assets not referred anymore. */
	 void Remove(const IPrimitiveComponent* Component, FRemovedRenderAssetArray* RemovedRenderAssets);

	/** Return a view of the data that has to be 100% thread safe. The content is allowed to be updated, but not memory must be reallocated. */
	const FRenderAssetInstanceView* GetAsyncView(bool bCreateIfNull);

	/** Returns whether we can add components to the manager. */
	bool CanAddComponent() const;

	/** Return the size taken for sub-allocation. */
	uint32 GetAllocatedSize() const;

	/** Apply specified offset to all cached primitives bounds */
	void OffsetBounds(const FVector& Offset);

protected:

	void OnRefreshVisibilityDone(int32 InBeginIndex, int32 InEndIndex);

private:

	typedef RenderAssetInstanceTask::FRefreshVisibilityTask FRefreshVisibilityTask;
	typedef RenderAssetInstanceTask::FNormalizeLightmapTexelFactorTask FNormalizeLightmapTexelFactorTask;

	struct FTasks
	{
		~FTasks() { SyncResults(); }
		void SyncResults();
		TRefCountPtr<FRefreshVisibilityTask> RefreshVisibilityTask;
		TRefCountPtr<FNormalizeLightmapTexelFactorTask> NormalizeLightmapTexelFactorTask;
	};

	/** The texture instances. Shared with the async task. */
	FRenderAssetInstanceStateTaskSync<FTasks> StateSync;

	/** A duplicate view for the async streaming task. */
	TRefCountPtr<const FRenderAssetInstanceView> AsyncView;

	/** Ranges from 0 to Bounds4Components.Num(). Used in the incremental update to update visibility. */
	int32 DirtyIndex;
};
