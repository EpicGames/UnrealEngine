// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	TextureStreamingManager.cpp: Implementation of content streaming classes.
=============================================================================*/

#include "Streaming/StreamingManagerTexture.h"

#include "Async/ParallelFor.h"
#include "Engine/Level.h"
#include "Engine/World.h"
#include "Engine/StaticMesh.h"
#include "Engine/SkeletalMesh.h"
#include "Engine/Texture2D.h"
#include "Engine/VolumeTexture.h"
#include "Engine/Texture2DArray.h"
#include "Materials/MaterialInterface.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/StringBuilder.h"
#include "RenderedTextureStats.h"
#include "UObject/UObjectIterator.h"
#include "DeviceProfiles/DeviceProfile.h"
#include "DeviceProfiles/DeviceProfileManager.h"
#include "ProfilingDebugging/CsvProfiler.h"
#include "RenderAssetUpdate.h"
#include "RenderingThread.h"
#include "Streaming/AsyncTextureStreaming.h"
#include "Components/PrimitiveComponent.h"
#include "Misc/CoreDelegates.h"
#include "TextureResource.h"
#include "HAL/LowLevelMemStats.h"
#include "ProfilingDebugging/MetadataTrace.h"
#include "Streaming/SimpleStreamableAssetManager.h"
#include "SceneInterface.h"
#include "Engine/Engine.h"

CSV_DECLARE_CATEGORY_MODULE_EXTERN(CORE_API, Basic);

CSV_DEFINE_CATEGORY(TextureStreaming, true);

#ifndef UE_STREAMINGRENDERASSETS_ARRAY_DEFAULT_RESERVED_SIZE
// The default size will reserve ~3MB, the element size is 168 bytes.
#define UE_STREAMINGRENDERASSETS_ARRAY_DEFAULT_RESERVED_SIZE 20000
#endif

static TAutoConsoleVariable<int32> CVarStreamingOverlapAssetAndLevelTicks(
	TEXT("r.Streaming.OverlapAssetAndLevelTicks"),
	!WITH_EDITOR,
	TEXT("Ticks render asset streaming info on a high priority task thread while ticking levels on GT"),
	ECVF_Default);

static TAutoConsoleVariable<int32> CVarStreamingAllowFastForceResident(
	TEXT("r.Streaming.AllowFastForceResident"),
	0,
	TEXT("Whether it is allowed to load in missing mips for fast-force-resident assets ASAP. ")
	TEXT("Useful to accelerate force-resident process but risks disturbing streaming metric calculation. ")
	TEXT("Fast-force-resident mips can't be sacrificed even when overbudget so use with caution."),
	ECVF_Default);

static int32 GAllowParallelUpdateStreamingRenderAssets = 0;
static FAutoConsoleVariableRef CVarStreamingAllowParallelUpdateStreamingRenderAssets(
	TEXT("r.Streaming.AllowParallelStreamingRenderAssets"),
	GAllowParallelUpdateStreamingRenderAssets,
	TEXT("Whether it is allowed to do UpdateStreamingRenderAssets with a ParallelFor to use more cores."),
	ECVF_Default);

static int32 GParallelRenderAssetsNumWorkgroups = 2;
static FAutoConsoleVariableRef CVarStreamingParallelRenderAssetsNumWorkgroups(
	TEXT("r.Streaming.ParallelRenderAssetsNumWorkgroups"),
	GParallelRenderAssetsNumWorkgroups,
	TEXT("How many workgroups we want to use for ParellelRenderAsset updates. ")
	TEXT("Splits the work up a bit more so we don't get as many waits. ")
	TEXT("Though adds overhead to GameThread if too high."),
	ECVF_Default);

static TAutoConsoleVariable<int32> CVarSyncStatesWhenBlocking(
	TEXT("r.Streaming.SyncStatesWhenBlocking"),
	0,
	TEXT("If true, SyncStates will be called to fully update async states before flushing outstanding streaming requests. Used by Movie Render Queue to ensure all streaming requests are handled each frame to avoid pop-in."),
	ECVF_Default);

// TODO: Remove once efficacy has been verified
static TAutoConsoleVariable<int32> CVarFlushDeferredMipLevelChangeCallbacksBeforeGC(
	TEXT("r.Streaming.FlushDeferredMipLevelChangeCallbacksBeforeGC"),
	1,
	TEXT("Whether to flush deferred mip level change callbacks before GC."),
	ECVF_Default);

static int32 GRenderAssetStreamingEnableStreamOutProtection = 1;
static FAutoConsoleVariableRef CVarStreamingEnableStreamOutProtection(
	TEXT("r.Streaming.EnableStreamOutProtection"),
	GRenderAssetStreamingEnableStreamOutProtection,
	TEXT("Whether streamable render assets can be added to the streaming manager's stream-out protection list."),
	ECVF_Default);

static int32 GRenderAssetStreamingEnableStrippingDuplicatePrimitiveComponent = 1;
static FAutoConsoleVariableRef CVarStreamingEnableStrippingDuplicatePrimitiveComponent(
	TEXT("r.Streaming.EnableStrippingDuplicatePrimitiveComponent"),
	GRenderAssetStreamingEnableStrippingDuplicatePrimitiveComponent,
	TEXT("Move UPrimitiveComponent from static instances to dynamic component manager when primitive is updated."),
	ECVF_Default);

// TODO: Remove once these calls have been proven safe in production
static TAutoConsoleVariable<int32> CVarProcessAddedRenderAssetsAfterAsyncWork(
	TEXT("r.Streaming.ProcessAddedRenderAssetsAfterAsyncWork"),
	1,
	TEXT("Whether to call ProcessAddedRenderAssets in subsqequent UpdateResourceStreaming stages after Async work has completed."),
	ECVF_Default);

namespace UE::Private::RenderAssetStreamingManager
{
	static bool bAllowParallelForDuringIncrementalUpdate = false;
	static FAutoConsoleVariableRef CVarAllowParallelForDuringIncrementalUpdate(
		TEXT("r.Streaming.AllowParallelRenderAssetStreamingManagerIncrementalUpdate"),
		bAllowParallelForDuringIncrementalUpdate,
		TEXT("Whether it is allowed for RenderAssetStreamingManager to use more cores using ParallelFor when doing the incremental update. ")
		TEXT("When enabled, texture streaming built data is not used to retrieve the primitive streaming render asset information."),
		ECVF_Default);

	static int32 GWorkerCountForParallelForDuringIncrementalUpdate = 0;
	static FAutoConsoleVariableRef CVarWorkerCountForParallelForDuringIncrementalUpdate(
		TEXT("r.Streaming.WorkerCountForParallelRenderAssetStreamingManagerIncrementalUpdate"),
		GWorkerCountForParallelForDuringIncrementalUpdate,
		TEXT("Set the max number of workers when r.Streaming.AllowParallelRenderAssetStreamingManagerIncrementalUpdate is enabled. ")
		TEXT("Only taked into account if value > 1 (GameThread counts as 1 worker). "),
		ECVF_Default);
}

bool TrackRenderAsset( const FString& AssetName );
bool UntrackRenderAsset( const FString& AssetName );
void ListTrackedRenderAssets( FOutputDevice& Ar, int32 NumTextures );

/**
 * Helper function to clamp the mesh to camera distance
 */
FORCEINLINE float ClampMeshToCameraDistanceSquared(float MeshToCameraDistanceSquared)
{
	// called from streaming thread, maybe even main thread
	return FMath::Max<float>(MeshToCameraDistanceSquared, 0.0f);
}

/*-----------------------------------------------------------------------------
	FRenderAssetStreamingManager implementation.
-----------------------------------------------------------------------------*/

/** Constructor, initializing all members and  */
FRenderAssetStreamingManager::FRenderAssetStreamingManager()
:	CurrentUpdateStreamingRenderAssetIndex(0)
,	AsyncWork( nullptr )
,	DynamicComponentManager([this](const FRemovedRenderAssetArray& RemovedRenderAssets) { SetRenderAssetsRemovedTimestamp(RemovedRenderAssets); })
,	CurrentPendingMipCopyRequestIdx(0)
,	LevelRenderAssetManagersLock(nullptr)
,	ProcessingStage( 0 )
,	NumRenderAssetProcessingStages(5)
,	bUseDynamicStreaming( false )
,	BoostPlayerTextures( 3.0f )
,	MemoryMargin(0)
,	EffectiveStreamingPoolSize(0)
,	MemoryOverBudget(0)
,	MaxEverRequired(0)
,	bPauseRenderAssetStreaming(false)
,	LastWorldUpdateTime(GIsEditor ? -FLT_MAX : 0) // In editor, visibility is not taken into consideration.
,	LastWorldUpdateTime_MipCalcTask(LastWorldUpdateTime)
{
	// Read settings from ini file.
	int32 TempInt;
	verify( GConfig->GetInt( TEXT("TextureStreaming"), TEXT("MemoryMargin"),				TempInt,						GEngineIni ) );
	MemoryMargin = TempInt;

	verify( GConfig->GetFloat( TEXT("TextureStreaming"), TEXT("LightmapStreamingFactor"),			GLightmapStreamingFactor,		GEngineIni ) );
	verify( GConfig->GetFloat( TEXT("TextureStreaming"), TEXT("ShadowmapStreamingFactor"),			GShadowmapStreamingFactor,		GEngineIni ) );

	int32 PoolSizeIniSetting = 0;
	GConfig->GetInt(TEXT("TextureStreaming"), TEXT("PoolSize"), PoolSizeIniSetting, GEngineIni);
	GConfig->GetBool(TEXT("TextureStreaming"), TEXT("UseDynamicStreaming"), bUseDynamicStreaming, GEngineIni);
	GConfig->GetFloat( TEXT("TextureStreaming"), TEXT("BoostPlayerTextures"), BoostPlayerTextures, GEngineIni );
	GConfig->GetBool(TEXT("TextureStreaming"), TEXT("NeverStreamOutRenderAssets"), GNeverStreamOutRenderAssets, GEngineIni);

	// -NeverStreamOutRenderAssets
	if (FParse::Param(FCommandLine::Get(), TEXT("NeverStreamOutRenderAssets")))
	{
		GNeverStreamOutRenderAssets = true;
	}
	if (GIsEditor)
	{
		// this would not be good or useful in the editor
		GNeverStreamOutRenderAssets = false;
	}
	if (GNeverStreamOutRenderAssets)
	{
		UE_LOGF(LogContentStreaming, Log, "Textures will NEVER stream out!");
	}

	// Convert from MByte to byte.
	MemoryMargin *= 1024 * 1024;

	for ( int32 LODGroup=0; LODGroup < TEXTUREGROUP_MAX; ++LODGroup )
	{
		const TextureGroup TextureLODGroup = (TextureGroup)LODGroup;
		const FTextureLODGroup& TexGroup = UDeviceProfileManager::Get().GetActiveProfile()->GetTextureLODSettings()->GetTextureLODGroup(TextureLODGroup);
		NumStreamedMips_Texture[LODGroup] = TexGroup.NumStreamedMips;
		// For compatibility reason, Character groups are always loaded with higher priority
		Settings.HighPriorityLoad_Texture[LODGroup] = TexGroup.HighPriorityLoad || TextureLODGroup == TEXTUREGROUP_Character || TextureLODGroup == TEXTUREGROUP_CharacterSpecular || TextureLODGroup == TEXTUREGROUP_CharacterNormalMap;
	}

	// TODO: NumStreamedMips_StaticMesh, NumStreamedMips_SkeletalMesh, NumStreamedMips_LandscapeMeshMobile
	NumStreamedMips_StaticMesh.Empty(1);
	NumStreamedMips_StaticMesh.Add(INT32_MAX);
	NumStreamedMips_SkeletalMesh.Empty(1);
	NumStreamedMips_SkeletalMesh.Add(INT32_MAX);

	// setup the streaming resource flush function pointer
	GFlushStreamingFunc = &FlushResourceStreaming;

	ProcessingStage = 0;
	AsyncWork = new FAsyncTask<FRenderAssetStreamingMipCalcTask>(this);

	RenderAssetInstanceAsyncWork = new RenderAssetInstanceTask::FDoWorkAsyncTask();

	if (!FSimpleStreamableAssetManager::IsEnabled())
	{
		DynamicComponentManager.RegisterTasks(RenderAssetInstanceAsyncWork->GetTask());
	}
	
	FCoreUObjectDelegates::GetPreGarbageCollectDelegate().AddRaw(this, &FRenderAssetStreamingManager::OnPreGarbageCollect);

	OnPakFileMounted2Handle = FCoreDelegates::GetOnPakFileMounted2().AddLambda([this](const IPakFile& PakFile)
	{
		InvalidateFileCache();
	});

	NewFileAddedDelegateHandle = FCoreDelegates::NewFileAddedDelegate.AddLambda([this](const FString& FileName)
	{
		MarkMountedStateDirty(MakeIoFilenameHash(FileName));
	});

	AsyncUnsafeStreamingRenderAssets.Reserve(UE_STREAMINGRENDERASSETS_ARRAY_DEFAULT_RESERVED_SIZE);
}

FRenderAssetStreamingManager::~FRenderAssetStreamingManager()
{
	AsyncWork->EnsureCompletion();
	delete AsyncWork;

	RenderAssetInstanceAsyncWork->EnsureCompletion();
	
	FCoreDelegates::GetOnPakFileMounted2().Remove(OnPakFileMounted2Handle);
	FCoreDelegates::NewFileAddedDelegate.Remove(NewFileAddedDelegateHandle);
	FCoreUObjectDelegates::GetPreGarbageCollectDelegate().RemoveAll(this);

	// Clear the stats
	DisplayedStats.Reset();
	STAT(DisplayedStats.Apply();)
}

TArray<FStreamingRenderAsset>& FRenderAssetStreamingManager::GetStreamingRenderAssetsAsyncSafe()
{
	if (StreamingRenderAssetsSyncEvent.IsValid()
		&& StreamingRenderAssetsSyncEvent->IsAwaitable()
	)
	{
		StreamingRenderAssetsSyncEvent->Wait(ENamedThreads::GameThread);
	}

	return AsyncUnsafeStreamingRenderAssets;
}

void FRenderAssetStreamingManager::OnPreGarbageCollect()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FRenderAssetStreamingManager::OnPreGarbageCollect);

	UE::TScopeLock ScopeLock(CriticalSection);

	if (StreamingRenderAssetsSyncEvent.IsValid())
	{
		StreamingRenderAssetsSyncEvent->Wait(ENamedThreads::GameThread);
	}

	QUICK_SCOPE_CYCLE_COUNTER(STAT_FRenderAssetStreamingManager_OnPreGarbageCollect);

	if (CVarFlushDeferredMipLevelChangeCallbacksBeforeGC.GetValueOnGameThread() != 0)
	{
		TickDeferredMipLevelChangeCallbacks();
	}

	if (FSimpleStreamableAssetManager::IsEnabled())
	{
		return;
	}

	FRemovedRenderAssetArray RemovedRenderAssets;

	// Check all levels for pending kills.
	check(!LevelRenderAssetManagersLock);
	for (int32 Index = 0; Index < LevelRenderAssetManagers.Num(); ++Index)
	{
		if (LevelRenderAssetManagers[Index] == nullptr)
		{
			continue;
		}

		FLevelRenderAssetManager& LevelManager = *LevelRenderAssetManagers[Index];
		if (!IsValid(LevelManager.GetLevel()))
		{
			LevelManager.Remove(&RemovedRenderAssets);

			// Remove the level entry. The async task view will still be valid as it uses a shared ptr.
			delete LevelRenderAssetManagers[Index];
			LevelRenderAssetManagers[Index] = nullptr;
		}
	}

	DynamicComponentManager.OnPreGarbageCollect(RemovedRenderAssets);

	SetRenderAssetsRemovedTimestamp(RemovedRenderAssets);
}

void FRenderAssetStreamingManager::MarkMountedStateDirty(FIoFilenameHash FilenameHash)
{
	if (!bRecacheAllFiles && FilenameHash != INVALID_IO_FILENAME_HASH)
	{
		LLM_SCOPE(ELLMTag::StreamingManager);
		LLM_TAGSET_SCOPE_CLEAR(ELLMTagSet::Assets);
		LLM_TAGSET_SCOPE_CLEAR(ELLMTagSet::AssetClasses);
		UE_TRACE_METADATA_CLEAR_SCOPE();
		FScopeLock ScopeLock(&MountedStateDirtyFilesCS);
		MountedStateDirtyFiles.Emplace(FilenameHash);
	}
}

/**
 * Cancels the timed Forced resources (i.e used the Kismet action "Stream In Textures").
 */
void FRenderAssetStreamingManager::CancelForcedResources()
{
	UE::TScopeLock ScopeLock(CriticalSection);

	TArray<FStreamingRenderAsset>& StreamingRenderAssets = GetStreamingRenderAssetsAsyncSafe();

	// Update textures/meshes that are Forced on a timer.
	for ( int32 Idx=0; Idx < StreamingRenderAssets.Num(); ++Idx )
	{
		FStreamingRenderAsset& StreamingRenderAsset = StreamingRenderAssets[ Idx ];

		// Make sure this streaming texture/mesh hasn't been marked for removal.
		if (StreamingRenderAsset.RenderAsset)
		{
			// Remove any prestream requests from textures/meshes
			float TimeLeft = (float)(StreamingRenderAsset.RenderAsset->ForceMipLevelsToBeResidentTimestamp - FApp::GetCurrentTime());
			if ( TimeLeft >= 0.0f )
			{
				StreamingRenderAsset.RenderAsset->SetForceMipLevelsToBeResident( -1.0f );
				StreamingRenderAsset.InstanceRemovedTimestamp = -FLT_MAX;
				StreamingRenderAsset.RenderAsset->InvalidateLastRenderTimeForStreaming();
#if STREAMING_LOG_CANCELFORCED
				UE_LOGF(LogContentStreaming, Log, "Canceling forced texture: %ls (had %.1f seconds left)", *StreamingRenderAsset.Texture->GetFullName(), TimeLeft );
#endif
			}
		}
	}

	// Reset the streaming system, so it picks up any changes to the asset right away.
	ProcessingStage = 0;
}

/**
 * Notifies manager of "level" change so it can prioritize character textures for a few frames.
 */
void FRenderAssetStreamingManager::NotifyLevelChange()
{
}

/** Don't stream world resources for the next NumFrames. */
void FRenderAssetStreamingManager::SetDisregardWorldResourcesForFrames( int32 NumFrames )
{
	//@TODO: We could perhaps increase the priority factor for character textures...
}

/**
 *	Try to stream out texture/mesh mip-levels to free up more memory.
 *	@param RequiredMemorySize	- Additional texture memory required
 *	@return						- Whether it succeeded or not
 **/
bool FRenderAssetStreamingManager::StreamOutRenderAssetData( int64 RequiredMemorySize )
{
	UE::TScopeLock ScopeLock(CriticalSection);
	LLM_SCOPE(ELLMTag::StreamingManager);
	LLM_TAGSET_SCOPE_CLEAR(ELLMTagSet::Assets);
	LLM_TAGSET_SCOPE_CLEAR(ELLMTagSet::AssetClasses);
	UE_TRACE_METADATA_CLEAR_SCOPE();

	const int64 MaxTempMemoryAllowed = static_cast<int64>(Settings.MaxTempMemoryAllowed) * 1024 * 1024;
	const bool CachedPauseTextureStreaming = bPauseRenderAssetStreaming;
	TArray<FStreamingRenderAsset>& StreamingRenderAssets = GetStreamingRenderAssetsAsyncSafe();

	// Pause texture streaming to prevent sending load requests.
	bPauseRenderAssetStreaming = true;
	SyncStates(true);

	// Sort texture/mesh, having those that should be dropped first.
	TArray<int32> PrioritizedRenderAssets;
	PrioritizedRenderAssets.Empty(StreamingRenderAssets.Num());
	for (int32 Idx = 0; Idx < StreamingRenderAssets.Num(); ++Idx)
	{
		FStreamingRenderAsset& StreamingRenderAsset = StreamingRenderAssets[Idx];
		// Only texture for which we can drop mips.
		if (StreamingRenderAsset.IsMaxResolutionAffectedByGlobalBias() && (!Settings.bFullyLoadMeshes || !StreamingRenderAsset.IsMesh()))
		{
			PrioritizedRenderAssets.Add(Idx);
		}
	}
	PrioritizedRenderAssets.Sort(FCompareRenderAssetByRetentionPriority(StreamingRenderAssets));

	int64 TempMemoryUsed = 0;
	int64 MemoryDropped = 0;

	// Process all texture/mesh, starting with the ones we least want to keep
	for (int32 PriorityIndex = PrioritizedRenderAssets.Num() - 1; PriorityIndex >= 0 && MemoryDropped < RequiredMemorySize; --PriorityIndex)
	{
		int32 RenderAssetIndex = PrioritizedRenderAssets[PriorityIndex];
		if (!StreamingRenderAssets.IsValidIndex(RenderAssetIndex)) continue;

		FStreamingRenderAsset& StreamingRenderAsset = StreamingRenderAssets[RenderAssetIndex];
		if (!StreamingRenderAsset.RenderAsset) continue;

		const int32 MinimalSize = StreamingRenderAsset.GetSize(StreamingRenderAsset.MinAllowedMips);
		const int32 CurrentSize = StreamingRenderAsset.GetSize(StreamingRenderAsset.ResidentMips);

		// Protection-aware StreamOut: held by external consumer (e.g. SkinWeightProfile readback)
		// returns false. Next eviction pass picks the asset up after release.
		if (StreamOutProtected(StreamingRenderAsset.RenderAsset, StreamingRenderAsset.MinAllowedMips))
		{
			MemoryDropped += CurrentSize - MinimalSize; 
			TempMemoryUsed += MinimalSize;

			StreamingRenderAsset.UpdateStreamingStatus();

			if (TempMemoryUsed >= MaxTempMemoryAllowed)
			{
				// Queue up the process on the render thread and wait for everything to complete.
				ENQUEUE_RENDER_COMMAND(FlushResourceCommand)(
					[](FRHICommandListImmediate& RHICmdList)
					{				
						RHICmdList.ImmediateFlush(EImmediateFlushType::FlushRHIThreadFlushResources);
					});
				FlushRenderingCommands();
				TempMemoryUsed = 0;
			}
		}
	}

	bPauseRenderAssetStreaming = CachedPauseTextureStreaming;
	UE_LOGF(LogContentStreaming, Log, "Streaming out texture memory! Saved %.2f MB.", float(MemoryDropped)/1024.0f/1024.0f);
	return true;
}

int64 FRenderAssetStreamingManager::GetPoolSize() const
{
	return GTexturePoolSize;
}

bool FRenderAssetStreamingManager::IsParallelForAllowedDuringIncrementalUpdate(int32 NumElementsToProcess, int32& OutParallelForMinBatchSize)
{
#if !WITH_EDITOR
	static int32 MinNumElementsToProcess = 8;
	bool bIsAllowed = UE::Private::RenderAssetStreamingManager::bAllowParallelForDuringIncrementalUpdate && FApp::ShouldUseThreadingForPerformance() && (NumElementsToProcess >= MinNumElementsToProcess);
	OutParallelForMinBatchSize = (bIsAllowed && (UE::Private::RenderAssetStreamingManager::GWorkerCountForParallelForDuringIncrementalUpdate > 0) && (NumElementsToProcess > UE::Private::RenderAssetStreamingManager::GWorkerCountForParallelForDuringIncrementalUpdate)) ? FMath::DivideAndRoundUp(NumElementsToProcess, UE::Private::RenderAssetStreamingManager::GWorkerCountForParallelForDuringIncrementalUpdate) : 1;
	return bIsAllowed;
#else
	// Some functions like UMaterialInterface::FindTextureStreamingDataIndexRange are not thread safe in editor
	OutParallelForMinBatchSize = 1;
	return false;
#endif
}

void FRenderAssetStreamingManager::IncrementalUpdate(float Percentage, bool bUpdateDynamicComponents)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FRenderAssetStreamingManager::IncrementalUpdate);
	QUICK_SCOPE_CYCLE_COUNTER(STAT_FRenderAssetStreamingManager_IncrementalUpdate);

	if (FSimpleStreamableAssetManager::IsEnabled())
	{
		FSimpleStreamableAssetManager::ProcessFallbackRegistrations_GameThread();
		return;
	}
	
	FRemovedRenderAssetArray RemovedRenderAssets;

	int64 NumStepsLeftForIncrementalBuild = CVarStreamingNumStaticComponentsProcessedPerFrame.GetValueOnGameThread();
	if (NumStepsLeftForIncrementalBuild <= 0) // When 0, don't allow incremental updates.
	{
		NumStepsLeftForIncrementalBuild = MAX_int64;
	}

	{
		// Prevent hazard if levels are added or removed during iteration
		FScopedLevelRenderAssetManagersLock ScopedLevelRenderAssetManagersLock(this);

		for (FLevelRenderAssetManager* LevelManager : LevelRenderAssetManagers)
		{
			if (LevelManager != nullptr)
			{
				LevelManager->IncrementalUpdate(DynamicComponentManager, RemovedRenderAssets, NumStepsLeftForIncrementalBuild, Percentage, bUseDynamicStreaming); // Complete the incremental update.
			}
		}
	}

	// Dynamic component are only udpated when it is useful for the dynamic async view.
	if (bUpdateDynamicComponents && bUseDynamicStreaming)
	{
		DynamicComponentManager.IncrementalUpdate(RemovedRenderAssets, Percentage);
	}

	SetRenderAssetsRemovedTimestamp(RemovedRenderAssets);
}

void FRenderAssetStreamingManager::TickFastResponseAssets()
{
	if (!CVarStreamingAllowFastForceResident.GetValueOnGameThread())
	{
		return;
	}

	TArray<FStreamingRenderAsset>& StreamingRenderAssets = GetStreamingRenderAssetsAsyncSafe();

	for (TSet<UStreamableRenderAsset*>::TIterator It(FastResponseRenderAssets); It; ++It)
	{
		UStreamableRenderAsset* RenderAsset = *It;
		
		if (!RenderAsset
			|| !RenderAsset->bIgnoreStreamingMipBias
			|| (!RenderAsset->bForceMiplevelsToBeResident && RenderAsset->ForceMipLevelsToBeResidentTimestamp < FApp::GetCurrentTime()))
		{
			// No longer qualified for fast response
			It.RemoveCurrent();
			continue;
		}

		if (RenderAsset->GetLastRenderTimeForStreaming() < LastWorldUpdateTime_MipCalcTask)
		{
			// Not visible
			continue;
		}

		const int32 StreamingIdx = RenderAsset->StreamingIndex;
		FStreamingRenderAsset& Asset = StreamingRenderAssets[StreamingIdx];

		check(RenderAsset == Asset.RenderAsset);

		Asset.UpdateStreamingStatus();

		if (Asset.ResidentMips == Asset.RequestedMips && Asset.ResidentMips < Asset.MaxAllowedMips)
		{
			RenderAsset->StreamIn(Asset.MaxAllowedMips, true);
			RenderAsset->bHasStreamingUpdatePending = true;
			Asset.SetFlag(FStreamingRenderAssetFlags::HasUpdatePending, true);
			VisibleFastResponseRenderAssetIndices.Add(StreamingIdx);

			for (int32 Idx = 0; Idx < PendingMipCopyRequests.Num(); ++Idx)
			{
				FPendingMipCopyRequest& PendingRequest = PendingMipCopyRequests[Idx];

				if (PendingRequest.RenderAsset == RenderAsset)
				{
					PendingRequest.RenderAsset = nullptr;
				}
			}

			Asset.UpdateStreamingStatus();
			TrackRenderAssetEvent(&Asset, RenderAsset, true, this);
		}
	}
}

void FRenderAssetStreamingManager::ProcessRemovedRenderAssets()
{
	TArray<FStreamingRenderAsset>& StreamingRenderAssets = GetStreamingRenderAssetsAsyncSafe();

	for (int32 AssetIndex : RemovedRenderAssetIndices)
	{
		// Remove swap all elements, until this entry has a valid texture/mesh.
		// This handles the case where the last element was also removed.
		while (StreamingRenderAssets.IsValidIndex(AssetIndex) && !StreamingRenderAssets[AssetIndex].RenderAsset)
		{
			StreamingRenderAssets.RemoveAtSwap(AssetIndex, EAllowShrinking::No);
		}

		if (StreamingRenderAssets.IsValidIndex(AssetIndex))
		{
			// Update the texture with its new index.
			int32& StreamingIdx = StreamingRenderAssets[AssetIndex].RenderAsset->StreamingIndex;

			if (VisibleFastResponseRenderAssetIndices.Remove(StreamingIdx))
			{
				VisibleFastResponseRenderAssetIndices.Add(AssetIndex);
			}

			StreamingIdx = AssetIndex;
		}
	}
	RemovedRenderAssetIndices.Empty();
}

void FRenderAssetStreamingManager::ProcessAddedRenderAssets()
{
	// Add new textures or meshes.
	TArray<FStreamingRenderAsset>& StreamingRenderAssets = GetStreamingRenderAssetsAsyncSafe();
	StreamingRenderAssets.Reserve(StreamingRenderAssets.Num() + PendingStreamingRenderAssets.Num());

	for (UStreamableRenderAsset* Asset : PendingStreamingRenderAssets)
	{
		// Could be null if it was removed after being added.
		if (Asset)
		{
			Asset->StreamingIndex = StreamingRenderAssets.Num();
			TConstArrayView<int32> NumStreamedMips = GetNumStreamedMipsArray(Asset->GetRenderAssetType());
			new (StreamingRenderAssets) FStreamingRenderAsset(Asset, NumStreamedMips, Settings);
		}
	}
	PendingStreamingRenderAssets.Reset();
}

void FRenderAssetStreamingManager::ConditionalUpdateStaticData()
{
	static float PreviousLightmapStreamingFactor = GLightmapStreamingFactor;
	static float PreviousShadowmapStreamingFactor = GShadowmapStreamingFactor;
	static FRenderAssetStreamingSettings PreviousSettings = Settings;

	if (PreviousLightmapStreamingFactor != GLightmapStreamingFactor || 
		PreviousShadowmapStreamingFactor != GShadowmapStreamingFactor || 
		PreviousSettings != Settings)
	{
		STAT(GatheredStats.SetupAsyncTaskCycles += FPlatformTime::Cycles();)
		// Update each texture static data.
		for (FStreamingRenderAsset& StreamingRenderAsset : AsyncUnsafeStreamingRenderAssets)
		{
			StreamingRenderAsset.UpdateStaticData(Settings);

			// When the material quality changes, some textures could stop being used.
			// Refreshing their removed timestamp ensures not texture ends up in the unkwown 
			// ref heuristic (which would force load them).
			if (PreviousSettings.MaterialQualityLevel != Settings.MaterialQualityLevel)
			{
				StreamingRenderAsset.InstanceRemovedTimestamp = FApp::GetCurrentTime();
			}
		}
		STAT(GatheredStats.SetupAsyncTaskCycles -= (int32)FPlatformTime::Cycles();)

#if !UE_BUILD_SHIPPING
		// Those debug settings are config that are not expected to change in-game.
		const bool bDebugSettingsChanged = 
			PreviousSettings.bUseMaterialData != Settings.bUseMaterialData ||
			PreviousSettings.bUseNewMetrics != Settings.bUseNewMetrics ||
			PreviousSettings.bUsePerTextureBias != Settings.bUsePerTextureBias || 
			PreviousSettings.MaxTextureUVDensity != Settings.MaxTextureUVDensity;
#else
		const bool bDebugSettingsChanged = false;
#endif

		// If the material quality changes, everything needs to be updated.
		if (!FSimpleStreamableAssetManager::IsEnabled()
				&& (bDebugSettingsChanged || PreviousSettings.MaterialQualityLevel != Settings.MaterialQualityLevel))
		{
			TArray<ULevel*, TInlineAllocator<32> > Levels;

			// RemoveLevel data
			check(!LevelRenderAssetManagersLock);
			for (FLevelRenderAssetManager* LevelManager : LevelRenderAssetManagers)
			{
				if (LevelManager!=nullptr)
				{
					Levels.Push(LevelManager->GetLevel());
					LevelManager->Remove(nullptr);
					delete LevelManager;
				}
			}
			LevelRenderAssetManagers.Empty();

			for (ULevel* Level : Levels)
			{
				AddLevel(Level);
			}

			// Reinsert dynamic components
			DynamicComponentManager.ForEachReferencedComponent(
				[this](const IPrimitiveComponent* PrimitiveComponent)
				{
					NotifyPrimitiveUpdated_Concurrent(PrimitiveComponent);
				});
		}

		// Update the cache variables.
		PreviousLightmapStreamingFactor = GLightmapStreamingFactor;
		PreviousShadowmapStreamingFactor = GShadowmapStreamingFactor;
		PreviousSettings = Settings;
	}
}

void FRenderAssetStreamingManager::ProcessLevelsToReferenceToStreamedTextures()
{
	if (FSimpleStreamableAssetManager::IsEnabled())
	{
		return;
	}
	
	// Iterate through levels and reference Levels to StreamedTexture if needed
	for (int32 LevelIndex = 0; LevelIndex < LevelRenderAssetManagers.Num(); ++LevelIndex)
	{
		if (LevelRenderAssetManagers[LevelIndex] == nullptr)
		{
			continue;
		}

		FLevelRenderAssetManager& LevelRenderAssetManager = *LevelRenderAssetManagers[LevelIndex];
		if (LevelRenderAssetManager.HasBeenReferencedToStreamedTextures())
		{
			continue;
		}

		const FRenderAssetInstanceView* View = LevelRenderAssetManager.GetRawAsyncView();
		if (View == nullptr)
		{
			continue;
		}

		LevelRenderAssetManager.SetReferencedToStreamedTextures();

		FRenderAssetInstanceView::FRenderAssetIterator RenderAssetIterator = LevelRenderAssetManager.GetRawAsyncView()->GetRenderAssetIterator();

		for (; RenderAssetIterator; ++RenderAssetIterator)
		{
			const UStreamableRenderAsset* RenderAsset = *RenderAssetIterator;
			if (RenderAsset == nullptr || !ReferencedRenderAssets.Contains(RenderAsset) || !AsyncUnsafeStreamingRenderAssets.IsValidIndex(RenderAsset->StreamingIndex))
			{
				continue;
			}

			FStreamingRenderAsset& StreamingRenderAsset = AsyncUnsafeStreamingRenderAssets[RenderAsset->StreamingIndex];

			check(StreamingRenderAsset.RenderAsset == RenderAsset);

			TBitArray<>& LevelIndexUsage = StreamingRenderAsset.LevelIndexUsage;

			if (LevelIndex >= (int)(LevelIndexUsage.Num()))
			{
				uint32 NumBits = LevelIndex + 1 - LevelIndexUsage.Num();
				LevelIndexUsage.Add(false, NumBits);
			}

			LevelIndexUsage[LevelIndex] = true;
		}
	}
}

void FRenderAssetStreamingManager::UpdatePendingStates(bool bUpdateDynamicComponents)
{
	CheckUserSettings();

	ProcessRemovedRenderAssets();
	ProcessAddedRenderAssets();

	Settings.Update();
	ConditionalUpdateStaticData();

	// Fully complete all pending update static data (newly loaded levels).
	// Dynamic bounds are not updated here since the async task uses the async view generated from the last frame.
	// this makes the current dynamic data fully dirty, and it will get refreshed iterativelly for the next full update.
	IncrementalUpdate(1.f, bUpdateDynamicComponents);

	if (FSimpleStreamableAssetManager::IsEnabled())
	{
		return;
	}
	
	if (bUpdateDynamicComponents)
	{
		DynamicComponentManager.PrepareAsyncView();
	}

	ProcessLevelsToReferenceToStreamedTextures();
}

/**
 * Adds new textures/meshes and level data on the gamethread (while the worker thread isn't active).
 */
void FRenderAssetStreamingManager::PrepareAsyncTask(bool bProcessEverything)
{
	FRenderAssetStreamingMipCalcTask& AsyncTask = AsyncWork->GetTask();
	FTextureMemoryStats Stats;
	RHIGetTextureMemoryStats(Stats);

	// TODO: Track memory allocated by mesh LODs

	// When processing all textures, we need unlimited budget so that textures get all at their required states.
	// Same when forcing stream-in, for which we want all used textures to be fully loaded 
	if (Stats.IsUsingLimitedPoolSize() && !bProcessEverything && !Settings.bFullyLoadUsedTextures)
	{
		const int64 TempMemoryBudget = static_cast<int64>(Settings.MaxTempMemoryAllowed) * 1024 * 1024;
		AsyncTask.Reset(Stats.TotalGraphicsMemory, Stats.StreamingMemorySize, Stats.TexturePoolSize, TempMemoryBudget, MemoryMargin);
	}
	else
	{
		// Temp must be smaller since membudget only updates if it has a least temp memory available.
		AsyncTask.Reset(0, Stats.StreamingMemorySize, MAX_int64, MAX_int64 / 2, 0);
	}
	AsyncTask.StreamingData.Init(CurrentViewInfos, LastWorldUpdateTime, LevelRenderAssetManagers, DynamicComponentManager);

	LastWorldUpdateTime_MipCalcTask = LastWorldUpdateTime;
}

/**
 * Temporarily boosts the streaming distance factor by the specified number.
 * This factor is automatically reset to 1.0 after it's been used for mip-calculations.
 */
void FRenderAssetStreamingManager::BoostTextures( AActor* Actor, float BoostFactor )
{
	UE::TScopeLock ScopeLock(CriticalSection);

	if ( Actor )
	{
		TArray<UTexture*> Textures;
		Textures.Empty( 32 );

		for (UActorComponent* Component : Actor->GetComponents())
		{
			UPrimitiveComponent* Primitive = Cast<UPrimitiveComponent>(Component);
			if (Primitive && Primitive->IsRegistered())
			{
				Textures.Reset();
				Primitive->GetUsedTextures( Textures, EMaterialQualityLevel::Num );
				for ( UTexture* Texture : Textures )
				{
					FStreamingRenderAsset* StreamingTexture = GetStreamingRenderAsset(Texture);
					if ( StreamingTexture )
					{
						StreamingTexture->DynamicBoostFactor = FMath::Max( StreamingTexture->DynamicBoostFactor, BoostFactor );
					}
				}
			}
		}
	}
}

FRenderAssetStreamingManager::FScopedLevelRenderAssetManagersLock::FScopedLevelRenderAssetManagersLock(FRenderAssetStreamingManager* InStreamingManager)
	: StreamingManager(InStreamingManager)
{
	check(!StreamingManager->LevelRenderAssetManagersLock);
	StreamingManager->LevelRenderAssetManagersLock = this;
}

FRenderAssetStreamingManager::FScopedLevelRenderAssetManagersLock::~FScopedLevelRenderAssetManagersLock()
{
	StreamingManager->ProcessPendingLevelManagers();
	StreamingManager->LevelRenderAssetManagersLock = nullptr;
}

void FRenderAssetStreamingManager::ProcessPendingLevelManagers()
{
	check(LevelRenderAssetManagersLock);

	LevelRenderAssetManagers.Append(LevelRenderAssetManagersLock->PendingAddLevelManagers);

	FRemovedRenderAssetArray RemovedRenderAssets;

	for (FLevelRenderAssetManager* LevelManager : LevelRenderAssetManagersLock->PendingRemoveLevelManagers)
	{
		LevelManager->Remove(&RemovedRenderAssets);
		// Delete the level manager. The async task view will still be valid as it is ref-counted.
		delete LevelManager;
	}
	SetRenderAssetsRemovedTimestamp(RemovedRenderAssets);
}

/** Adds a ULevel to the streaming manager. This is called from 2 paths : after PostPostLoad and after AddToWorld */
void FRenderAssetStreamingManager::AddLevel( ULevel* Level )
{
	if (FSimpleStreamableAssetManager::IsEnabled())
	{
		return;
	}

	UE::TScopeLock ScopeLock(CriticalSection);
	LLM_SCOPE(ELLMTag::StreamingManager);
	LLM_TAGSET_SCOPE_CLEAR(ELLMTagSet::Assets);
	LLM_TAGSET_SCOPE_CLEAR(ELLMTagSet::AssetClasses);
	UE_TRACE_METADATA_CLEAR_SCOPE();

	check(Level);

	if (GIsEditor)
	{
		// In editor, we want to rebuild everything from scratch as the data could be changing.
		// To do so, we remove the level and reinsert it.
		RemoveLevel(Level);
	}
	else
	{
		// In game, because static components can not be changed, the level static data is computed and kept as long as the level is not destroyed.
		for (const FLevelRenderAssetManager* LevelManager : LevelRenderAssetManagers)
		{
			if (LevelManager!=nullptr && LevelManager->GetLevel() == Level)
			{
				// Nothing to do, since the incremental update automatically manages what needs to be done.
				return;
			}
		}

		if (LevelRenderAssetManagersLock)
		{
			for (const FLevelRenderAssetManager* LevelManager : LevelRenderAssetManagersLock->PendingAddLevelManagers)
			{
				check(LevelManager);
				if (LevelManager->GetLevel() == Level)
				{
					// Nothing to do, since the incremental update automatically manages what needs to be done.
					return;
				}
			}
		}
	}

	// If the level was not already there, create a new one, find an available slot or add a new one.
	RenderAssetInstanceAsyncWork->EnsureCompletion();
	FLevelRenderAssetManager* LevelRenderAssetManager = new FLevelRenderAssetManager(Level, RenderAssetInstanceAsyncWork->GetTask());

	uint32 LevelIndex = LevelRenderAssetManagers.FindLastByPredicate([](FLevelRenderAssetManager* Ptr) { return (Ptr == nullptr); });
	if (LevelIndex != INDEX_NONE)
	{
		LevelRenderAssetManagers[LevelIndex] = LevelRenderAssetManager;
	}
	else if (LevelRenderAssetManagersLock)
	{
		// Cannot add during recursion as it may cause the array to realloc and invalidate the iterator
		LevelRenderAssetManagersLock->PendingAddLevelManagers.Add(LevelRenderAssetManager);
	}
	else
	{
		LevelRenderAssetManagers.Add(LevelRenderAssetManager);
	}
}

/** Removes a ULevel from the streaming manager. */
void FRenderAssetStreamingManager::RemoveLevel( ULevel* Level )
{
	if (FSimpleStreamableAssetManager::IsEnabled())
	{
		return;
	}
	UE::TScopeLock ScopeLock(CriticalSection);

	check(Level);

	// In editor we remove levels when visibility changes, while in game we want to kept the static data as long as possible.
	// FLevelRenderAssetManager::IncrementalUpdate will remove dynamic components and mark textures/meshes timestamps.
	if (GIsEditor || !IsValid(Level) || Level->HasAnyFlags(RF_BeginDestroyed|RF_FinishDestroyed))
	{
		for (int32 Index = 0; Index < LevelRenderAssetManagers.Num(); ++Index)
		{
			FLevelRenderAssetManager* LevelManager = LevelRenderAssetManagers[Index];
			if (LevelManager!=nullptr && LevelManager->GetLevel() == Level)
			{
				LevelRenderAssetManagers[Index] = nullptr;

				if (LevelRenderAssetManagersLock)
				{
					// Do not delete during recursion in case the level manager is updating its internal states
					LevelRenderAssetManagersLock->PendingRemoveLevelManagers.Add(LevelManager);
				}
				else
				{
					FRemovedRenderAssetArray RemovedRenderAssets;
					LevelManager->Remove(&RemovedRenderAssets);
					SetRenderAssetsRemovedTimestamp(RemovedRenderAssets);

					// Delete the level manager. The async task view will still be valid as it is ref-counted.
					delete LevelManager;
				}
				break;
			}
		}
	}
}

void FRenderAssetStreamingManager::NotifyLevelOffset(ULevel* Level, const FVector& Offset)
{
	if (FSimpleStreamableAssetManager::IsEnabled())
	{
		return;
	}
	
	UE::TScopeLock ScopeLock(CriticalSection);

	for (FLevelRenderAssetManager* LevelManager : LevelRenderAssetManagers)
	{
		if (LevelManager!=nullptr && LevelManager->GetLevel() == Level)
		{
			LevelManager->NotifyLevelOffset(Offset);
			break;
		}
	}
}

void FRenderAssetStreamingManager::AddStreamingRenderAsset(UStreamableRenderAsset* InAsset)
{
	UE::TScopeLock ScopeLock(CriticalSection);
	LLM_SCOPE(ELLMTag::StreamingManager);
	LLM_TAGSET_SCOPE_CLEAR(ELLMTagSet::Assets);
	LLM_TAGSET_SCOPE_CLEAR(ELLMTagSet::AssetClasses);
	UE_TRACE_METADATA_CLEAR_SCOPE();

	STAT(GatheredStats.CallbacksCycles = -(int32)FPlatformTime::Cycles();)

	// Adds the new texture/mesh to the Pending list, to avoid reallocation of the thread-safe StreamingRenderAssets array.
	check(InAsset->StreamingIndex == INDEX_NONE);
	InAsset->StreamingIndex = PendingStreamingRenderAssets.Add(InAsset);

	// Mark as pending update while the streamer has not determined the required resolution (unless paused)
	InAsset->bHasStreamingUpdatePending = !bPauseRenderAssetStreaming;

	// Notify that this texture/mesh ptr is valid.
	ReferencedRenderAssets.Add(InAsset);

	STAT(GatheredStats.CallbacksCycles += FPlatformTime::Cycles();)
}

/**
 * Removes a texture/mesh from the streaming manager.
 */
void FRenderAssetStreamingManager::RemoveStreamingRenderAsset( UStreamableRenderAsset* RenderAsset )
{
	// FSimpleStreamableAssetManager::UnregisterAsset can be called outside critical Section
	FSimpleStreamableAssetManager::UnregisterAsset(RenderAsset);
	
	UE::TScopeLock ScopeLock(CriticalSection);
	LLM_SCOPE(ELLMTag::StreamingManager);
	LLM_TAGSET_SCOPE_CLEAR(ELLMTagSet::Assets);
	LLM_TAGSET_SCOPE_CLEAR(ELLMTagSet::AssetClasses);
	UE_TRACE_METADATA_CLEAR_SCOPE();
	TArray<FStreamingRenderAsset>& StreamingRenderAssets = GetStreamingRenderAssetsAsyncSafe();

	STAT(GatheredStats.CallbacksCycles = -(int32)FPlatformTime::Cycles();)

	const int32	Idx = RenderAsset->StreamingIndex;

	// Remove it from the Pending list if it is there.
	if (PendingStreamingRenderAssets.IsValidIndex(Idx) && PendingStreamingRenderAssets[Idx] == RenderAsset)
	{
		PendingStreamingRenderAssets[Idx] = nullptr;
	}
	else if (StreamingRenderAssets.IsValidIndex(Idx) && StreamingRenderAssets[Idx].RenderAsset == RenderAsset)
	{
		StreamingRenderAssets[Idx].RenderAsset = nullptr;
		RemovedRenderAssetIndices.Add(Idx);

		FastResponseRenderAssets.Remove(RenderAsset);
		VisibleFastResponseRenderAssetIndices.Remove(Idx);
	}

	RenderAsset->StreamingIndex = INDEX_NONE;
	RenderAsset->bHasStreamingUpdatePending = false;

	// Remove reference to this texture/mesh.
	ReferencedRenderAssets.Remove(RenderAsset);


	{
		// Ensure the protected array can never hold a stale pointer.
		FRWScopeLock Lock(StreamOutProtectedAssetsLock, SLT_Write);
		StreamOutProtectedAssets.RemoveSingleSwap(RenderAsset, EAllowShrinking::No);
	}

	STAT(GatheredStats.CallbacksCycles += FPlatformTime::Cycles();)
}

bool FRenderAssetStreamingManager::TryRegisterStreamOutProtectedAsset(UStreamableRenderAsset* Asset)
{
	if (!Asset)
	{
		return false;
	}

	FRWScopeLock Lock(StreamOutProtectedAssetsLock, SLT_Write);

	// HasPendingInitOrStreaming() is tested under the same lock that StreamOut paths
	// take (read), so concurrent StreamOut paths are either:
	//  - blocked waiting for this write lock and will see Asset in the array, or
	//  - already finished — in which case HasPendingInitOrStreaming() reports true
	//    and we bail.
	if (Asset->HasPendingInitOrStreaming())
	{
		return false;
	}

	if (GRenderAssetStreamingEnableStreamOutProtection)
	{
		StreamOutProtectedAssets.AddUnique(Asset);
	}
	return true;
}

void FRenderAssetStreamingManager::UnregisterStreamOutProtectedAsset(UStreamableRenderAsset* Asset)
{
	if (!Asset)
	{
		return;
	}
	FRWScopeLock Lock(StreamOutProtectedAssetsLock, SLT_Write);
	StreamOutProtectedAssets.RemoveSingleSwap(Asset, EAllowShrinking::No);
}

bool FRenderAssetStreamingManager::StreamOutProtected(UStreamableRenderAsset* Asset, int32 NewMipCount)
{
	FRWScopeLock Lock(StreamOutProtectedAssetsLock, SLT_ReadOnly);
	if (StreamOutProtectedAssets.Contains(Asset))
	{
		return false;
	}
	return Asset->StreamOut(NewMipCount);
}

bool FRenderAssetStreamingManager::IsFullyStreamedIn(UStreamableRenderAsset* RenderAsset)
{
	check(RenderAsset);

	const FStreamingRenderAsset* StreamingAsset = GetStreamingRenderAsset(RenderAsset);
	const FStreamableRenderResourceState& AssetState = RenderAsset->GetStreamableResourceState();
	
	if (StreamingAsset)
	{
		const FStreamingRenderAsset::EOptionalMipsState OptionalLODState = StreamingAsset->OptionalMipsState;
		int32 NumLODsToConsiderAsFull = AssetState.MaxNumLODs - RenderAsset->GetCachedLODBias();
		
		if (OptionalLODState == FStreamingRenderAsset::OMS_NoOptionalMips)
		{
			NumLODsToConsiderAsFull = FMath::Min(NumLODsToConsiderAsFull, (int32)AssetState.NumNonOptionalLODs);
		}

		return AssetState.NumResidentLODs >= NumLODsToConsiderAsFull;
	}

	return false;
}

/** Called when a spawned primitive is deleted, or when an actor is destroyed in the editor. */
void FRenderAssetStreamingManager::NotifyActorDestroyed( AActor* Actor )
{
	if (FSimpleStreamableAssetManager::IsEnabled())
	{
		return;
	}
	
	UE::TScopeLock ScopeLock(CriticalSection);

	STAT(GatheredStats.CallbacksCycles = -(int32)FPlatformTime::Cycles();)
	FRemovedRenderAssetArray RemovedRenderAssets;
	check(Actor);

	TInlineComponentArray<UPrimitiveComponent*> Components;
	Actor->GetComponents(Components);
	Components.Remove(nullptr);

	// Here we assume that level can not be changed in game, to allow an optimized path.
	ULevel* Level = !GIsEditor ? Actor->GetLevel() : nullptr;

	// Remove any reference in the level managers.
	check(!LevelRenderAssetManagersLock);
	for (FLevelRenderAssetManager* LevelManager : LevelRenderAssetManagers)
	{
		if (LevelManager!=nullptr && (!Level || LevelManager->GetLevel() == Level))
		{
			LevelManager->RemoveActorReferences(Actor);
			for (UPrimitiveComponent* Component : Components)
			{
				LevelManager->RemoveComponentReferences(Component->GetPrimitiveComponentInterface(), RemovedRenderAssets);
			}
		}
	}

	for (UPrimitiveComponent* Component : Components)
	{
		// Remove any references in the dynamic component manager.
		DynamicComponentManager.Remove(Component->GetPrimitiveComponentInterface(), &RemovedRenderAssets);

		// Reset this now as we have finished iterating over the levels
		Component->SetAttachedToStreamingManagerAsStatic(false);
	}

	SetRenderAssetsRemovedTimestamp(RemovedRenderAssets);
	STAT(GatheredStats.CallbacksCycles += FPlatformTime::Cycles();)
}

void FRenderAssetStreamingManager::RemoveStaticReferences(const IPrimitiveComponent* Primitive)
{
	if (FSimpleStreamableAssetManager::IsEnabled())
	{
		return;
	}
	
	UE::TScopeLock ScopeLock(CriticalSection);

	check(Primitive);
	FRenderAssetOwnerStreamingState& StreamingState = Primitive->GetStreamingState();

	if (StreamingState.IsAttachedToStreamingManagerAsStatic())
	{
		FRemovedRenderAssetArray RemovedRenderAssets;
		ULevel* Level = Primitive->GetComponentLevel();
		check(!LevelRenderAssetManagersLock);
		for (FLevelRenderAssetManager* LevelManager : LevelRenderAssetManagers)
		{
			if (LevelManager != nullptr && (!Level || LevelManager->GetLevel() == Level))
			{
				LevelManager->RemoveComponentReferences(Primitive, RemovedRenderAssets);
			}
		}
		StreamingState.SetAttachedToStreamingManagerAsStatic(false);
		// Nothing to do with removed textures/meshes as we are about to reinsert
	}
}

/**
 * Called when a primitive is detached from an actor or another component.
 * Note: We should not be accessing the primitive or the UTexture after this call!
 */
void FRenderAssetStreamingManager::NotifyPrimitiveDetached( const IPrimitiveComponent* Primitive )
{
	if (FSimpleStreamableAssetManager::IsEnabled())
	{
		return;
	}
	
	UE::TScopeLock ScopeLock(CriticalSection);

	if (!Primitive || !Primitive->GetStreamingState().IsAttachedToStreamingManager())
	{
		return;
	}

	STAT(GatheredStats.CallbacksCycles = -(int32)FPlatformTime::Cycles();)
	FRemovedRenderAssetArray RemovedRenderAssets;

#if STREAMING_LOG_DYNAMIC
		UE_LOGF(LogContentStreaming, Log, "NotifyPrimitiveDetached(0x%08x \"%ls\"), IsRegistered=%d", SIZE_T(Primitive), *Primitive->GetReadableName(), Primitive->IsRegistered());
#endif

	FRenderAssetOwnerStreamingState& StreamingState = Primitive->GetStreamingState();

	if (StreamingState.IsAttachedToStreamingManagerAsStatic())
	{
		// Here we assume that level can not be changed in game, to allow an optimized path.
		// If there is not level, then we assume it could be in any level.
		ULevel* Level = !GIsEditor ? Primitive->GetComponentLevel() : nullptr;
		if (Level && (!IsValid(Level) || Level->HasAnyFlags(RF_BeginDestroyed|RF_FinishDestroyed)))
		{
			// Do a batch remove to prevent handling each component individually.
			RemoveLevel(Level);
		}
		else
		{
			const UObject* PrimitiveObject = Primitive->GetUObject();
			// Unless in editor, we don't want to remove reference in static level data when toggling visibility.
			if (GIsEditor || !IsValid(PrimitiveObject) || PrimitiveObject->HasAnyFlags(RF_BeginDestroyed | RF_FinishDestroyed))
			{
				check(!LevelRenderAssetManagersLock);
				for (FLevelRenderAssetManager* LevelManager : LevelRenderAssetManagers)
				{
					if (LevelManager != nullptr && (!Level || LevelManager->GetLevel() == Level))
					{
						LevelManager->RemoveComponentReferences(Primitive, RemovedRenderAssets);
					}
				}
				StreamingState.SetAttachedToStreamingManagerAsStatic(false);
			}
		}
	}
	
	// Dynamic component must be removed when visibility changes.
	DynamicComponentManager.Remove(Primitive, &RemovedRenderAssets);

	SetRenderAssetsRemovedTimestamp(RemovedRenderAssets);
	STAT(GatheredStats.CallbacksCycles += FPlatformTime::Cycles();)
}

/**
* Mark the textures/meshes with a timestamp. They're about to lose their location-based heuristic and we don't want them to
* start using LastRenderTime heuristic for a few seconds until they are garbage collected!
*
* @param RemovedRenderAssets	List of removed textures or meshes.
*/
void FRenderAssetStreamingManager::SetRenderAssetsRemovedTimestamp(const FRemovedRenderAssetArray& RemovedRenderAssets)
{
	const double CurrentTime = FApp::GetCurrentTime();
	for ( int32 Idx=0; Idx < RemovedRenderAssets.Num(); ++Idx )
	{
		// When clearing references to textures/meshes, those textures/meshes could be already deleted.
		// This happens because we don't clear texture/mesh references in RemoveStreamingRenderAsset.
		const UStreamableRenderAsset* Asset = RemovedRenderAssets[Idx];
		if (!ReferencedRenderAssets.Contains(Asset)) continue;

		FStreamingRenderAsset* StreamingRenderAsset = GetStreamingRenderAsset(Asset);
		if (StreamingRenderAsset)
		{
			StreamingRenderAsset->InstanceRemovedTimestamp = CurrentTime;
		}
	}
}

void FRenderAssetStreamingManager::InvalidateFileCache()
{
	FScopeLock ScopeLock(&MountedStateDirtyFilesCS);
	bRecacheAllFiles = true;
	MountedStateDirtyFiles.Empty();
}

void FRenderAssetStreamingManager::NotifyPrimitiveUpdated( const IPrimitiveComponent* Primitive )
{
	if (FSimpleStreamableAssetManager::IsEnabled() || !Primitive)
	{
		return;
	}
	
	STAT(GatheredStats.CallbacksCycles = -(int32)FPlatformTime::Cycles();)

	FRenderAssetOwnerStreamingState& StreamingState = Primitive->GetStreamingState();

	// This can sometime be called from async threads if actor constructor ends up calling SetStaticMesh, for example.
	// When this happens, the states will be initialized when the components render states will be set.
	if (IsInGameThread() && bUseDynamicStreaming && Primitive && !StreamingState.IsIgnoreStreamingManagerUpdate())
	{
		UE::TScopeLock ScopeLock(CriticalSection);

		// Check if there is a pending renderstate update, useful since streaming data can be updated in UPrimitiveComponent::CreateRenderState_Concurrent().
		// We handle this here to prevent the primitive from being updated twice in the same frame.
		const bool bHasRenderStateUpdateScheduled = !Primitive->IsRegistered() || !Primitive->IsRenderStateCreated() || Primitive->IsRenderStateDirty();
		bool bUpdatePrimitive = false;

		if (StreamingState.IsHandledByStreamingManagerAsDynamic())
		{
			// If an update is already scheduled and it is already handled as dynamic, nothing to do.
			bUpdatePrimitive = !bHasRenderStateUpdateScheduled;
		}
		else if (StreamingState.IsAttachedToStreamingManagerAsStatic())
		{
			// Change this primitive from being handled as static to being handled as dynamic.
			// This is required because the static data can not be updated.
			RemoveStaticReferences(Primitive);

			StreamingState.SetHandledByStreamingManagerAsDynamic(true);
			bUpdatePrimitive = !bHasRenderStateUpdateScheduled;
		}
		else
		{
			// If neither flag are set, NotifyPrimitiveUpdated() was called on a new primitive, which will be updated correctly when its render state gets created.
			// Don't force a dynamic update here since a static primitive can still go through the static path at this point.
		}

		if (bUpdatePrimitive)
		{
			FStreamingTextureLevelContext LevelContext(GetCurrentMaterialQualityLevelChecked(), Primitive);
			DynamicComponentManager.Add(Primitive, LevelContext);
		}
	}

	STAT(GatheredStats.CallbacksCycles += FPlatformTime::Cycles();)
}

/**
 * Called when a primitive has had its textures/mesh changed.
 * Only affects primitives that were already attached.
 * Replaces previous info.
 */
void FRenderAssetStreamingManager::NotifyPrimitiveUpdated_Concurrent( const IPrimitiveComponent* Primitive )
{
	if (FSimpleStreamableAssetManager::IsEnabled())
	{
		return;
	}
	
	STAT(int32 CallbackCycle = -(int32)FPlatformTime::Cycles();)

	// The level context is not used currently.
	if (bUseDynamicStreaming && Primitive)
	{
		UE::TScopeLock ScopeLock(CriticalSection);
		FStreamingTextureLevelContext LevelContext(GetCurrentMaterialQualityLevelChecked());
		if (DynamicComponentManager.Add(Primitive, LevelContext) == EAddComponentResult::Success)
		{
			if (GRenderAssetStreamingEnableStrippingDuplicatePrimitiveComponent>0)
			{
				RemoveStaticReferences(Primitive);
			}
		}
	}

	STAT(CallbackCycle += (int32)FPlatformTime::Cycles();)

	UE_AUTORTFM_OPEN
	{
		STAT(FPlatformAtomics::InterlockedAdd(&GatheredStats.CallbacksCycles, CallbackCycle));
	};
}

void FRenderAssetStreamingManager::SyncStates(bool bCompleteFullUpdateCycle)
{
	// Finish the current update cycle. 
	while (ProcessingStage != 0 && bCompleteFullUpdateCycle)
	{
		UpdateResourceStreaming(0, false);
	}

	// Wait for async tasks
	AsyncWork->EnsureCompletion();
	RenderAssetInstanceAsyncWork->EnsureCompletion();

	// Update any pending states, including added/removed textures/meshes.
	// Doing so when ProcessingStage != 0 risk invalidating the indices in the async task used in StreamRenderAssets().
	// This would in practice postpone some of the load and cancel requests.
	UpdatePendingStates(false);
}

/**
 * Returns the corresponding FStreamingRenderAsset for a texture or mesh.
 */
FStreamingRenderAsset* FRenderAssetStreamingManager::GetStreamingRenderAsset( const UStreamableRenderAsset* RenderAsset )
{
	UE::TScopeLock ScopeLock(CriticalSection);

	if (RenderAsset && AsyncUnsafeStreamingRenderAssets.IsValidIndex(RenderAsset->StreamingIndex))
	{
		FStreamingRenderAsset* StreamingRenderAsset = &AsyncUnsafeStreamingRenderAssets[RenderAsset->StreamingIndex];

		// If the texture/mesh don't match, this means the texture/mesh is pending in PendingStreamingRenderAssets, for which no FStreamingRenderAsset* is yet allocated.
		// If this is not acceptable, the caller should first synchronize everything through SyncStates
		return StreamingRenderAsset->RenderAsset == RenderAsset ? StreamingRenderAsset : nullptr;
	}
	else
	{
		return nullptr;
	}
}

/**
 * Updates streaming for an individual texture/mesh, taking into account all view infos.
 *
 * @param RenderAsset	Texture or mesh to update
 */
void FRenderAssetStreamingManager::UpdateIndividualRenderAsset( UStreamableRenderAsset* RenderAsset )
{
	UE::TScopeLock ScopeLock(CriticalSection);

	if (!IStreamingManager::Get().IsStreamingEnabled() || !RenderAsset) return;

	// Because we want to priorize loading of this texture, 
	// don't process everything as this would send load requests for all textures.
	SyncStates(false);

	FStreamingRenderAsset* StreamingRenderAsset = GetStreamingRenderAsset(RenderAsset);
	if (!StreamingRenderAsset) return;

	const TConstArrayView<int32> NumStreamedMips = GetNumStreamedMipsArray(StreamingRenderAsset->RenderAssetType);
	StreamingRenderAsset->UpdateDynamicData(NumStreamedMips, Settings);

	if (StreamingRenderAsset->GetFlag(FStreamingRenderAssetFlags::ForceFullyLoad)) // Somewhat expected at this point.
	{
		StreamingRenderAsset->WantedMips = StreamingRenderAsset->BudgetedMips = StreamingRenderAsset->MaxAllowedMips;
	}

	StreamingRenderAsset->StreamWantedMips(*this);
}

bool FRenderAssetStreamingManager::FastForceFullyResident(UStreamableRenderAsset* RenderAsset)
{
	check(IsInGameThread());
	TArray<FStreamingRenderAsset>& StreamingRenderAssets = GetStreamingRenderAssetsAsyncSafe();

	if (CVarStreamingAllowFastForceResident.GetValueOnGameThread()
		&& IStreamingManager::Get().IsStreamingEnabled()
		&& !bPauseRenderAssetStreaming
		&& RenderAsset
		&& RenderAsset->bIgnoreStreamingMipBias
		&& (RenderAsset->bForceMiplevelsToBeResident || RenderAsset->ForceMipLevelsToBeResidentTimestamp >= FApp::GetCurrentTime())
		&& StreamingRenderAssets.IsValidIndex(RenderAsset->StreamingIndex)
		&& StreamingRenderAssets[RenderAsset->StreamingIndex].RenderAsset == RenderAsset)
	{
		const int32 StreamingIdx = RenderAsset->StreamingIndex;
		FStreamingRenderAsset& Asset = StreamingRenderAssets[StreamingIdx];

		Asset.UpdateStreamingStatus();

		if (Asset.ResidentMips < Asset.MaxAllowedMips)
		{
			FastResponseRenderAssets.Add(Asset.RenderAsset);
			return true;
		}
	}
	return false;
}

/**
 * Not thread-safe: Updates a portion (as indicated by 'StageIndex') of all streaming textures,
 * allowing their streaming state to progress.
 *
 * @param Context			Context for the current stage (frame)
 * @param StageIndex		Current stage index
 * @param NumUpdateStages	Number of texture update stages
 */
void FRenderAssetStreamingManager::UpdateStreamingRenderAssets(int32 StageIndex, int32 NumUpdateStages, bool bAsync)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UpdateStreamingRenderAssets);
	if ( StageIndex == 0 )
	{
		CurrentUpdateStreamingRenderAssetIndex = 0;
		InflightRenderAssets.Reset();
	}

	int32 StartIndex = CurrentUpdateStreamingRenderAssetIndex;
	int32 EndIndex = AsyncUnsafeStreamingRenderAssets.Num() * (StageIndex + 1) / NumUpdateStages;

	if (GAllowParallelUpdateStreamingRenderAssets)
	{
		struct FPacket
		{
			FPacket(int32 InStartIndex, int32 InEndIndex, TArrayView<FStreamingRenderAsset> InStreamingRenderAssets)
				: StartIndex(InStartIndex)
				, EndIndex(InEndIndex)
				, StreamingRenderAssets(InStreamingRenderAssets)
			{
			}

			// Inputs
			int32 StartIndex;
			int32 EndIndex;
			TArrayView<FStreamingRenderAsset> StreamingRenderAssets;

			// Outputs
			TArray<int32> LocalInflightRenderAssets;
			TArray<UStreamableRenderAsset*> LocalDeferredTickCBAssets;

			STAT(int64 MipIOBandwidth = 0;)

			char FalseSharingSpacerBuffer[PLATFORM_CACHE_LINE_SIZE];		// separate Packets to avoid false sharing
		};

		// In order to pass the InflightRenderAssets and DeferredTickCBAssets to each parallel for loop we need to break this into a number of workgroups.
		// Cannot be too large or the overhead of consolidating all the arrays will take too long.  Cannot be too small or the parallel for will not have
		// enough work.  Can be adjusted by CVarStreamingParallelRenderAssetsNumWorkgroups.
		int32 Num = EndIndex - StartIndex;
		int32 NumThreadTasks = FMath::Min<int32>(FMath::Max(1, FTaskGraphInterface::Get().GetNumWorkerThreads()) * GParallelRenderAssetsNumWorkgroups, Num - 1);
		// make sure it never goes to zero, as a divide by zero could happen below
		NumThreadTasks = FMath::Max<int32>(NumThreadTasks, 1);
		TArray<FPacket> Packets;
		Packets.Reset(NumThreadTasks); // Go ahead and reserve space up front
		int32 Start = StartIndex;
		int32 NumRemaining = Num;
		int32 NumItemsPerGroup = (Num / NumThreadTasks) + 1;
		for (int32 i = 0; i < NumThreadTasks; ++i)
		{
			int32 NumAssetsToProcess = FMath::Min<int32>(NumRemaining, NumItemsPerGroup);
			Packets.Emplace(Start, Start + NumAssetsToProcess, AsyncUnsafeStreamingRenderAssets);

			Start += NumAssetsToProcess;
			NumRemaining -= NumAssetsToProcess;
			if (NumRemaining <= 0)
			{
				break;
			}
		}

		ParallelFor(TEXT("RenderAssetStreaming"), Packets.Num(), 1, [this, &Packets](int32 PacketIndex)
		{
			FTaskTagScope Scope(ETaskTag::EParallelGameThread);

			for (int32 Index = Packets[PacketIndex].StartIndex; Index < Packets[PacketIndex].EndIndex; ++Index)
			{
				FStreamingRenderAsset& StreamingRenderAsset = Packets[PacketIndex].StreamingRenderAssets[Index];

				// Is this texture/mesh marked for removal? Will get cleanup once the async task is done.
				if (!StreamingRenderAsset.RenderAsset)
				{
					continue;
				}

				STAT(int32 PreviousResidentMips = StreamingRenderAsset.ResidentMips;)

				const TConstArrayView<int32> NumStreamedMips = GetNumStreamedMipsArray(StreamingRenderAsset.RenderAssetType);

				StreamingRenderAsset.UpdateDynamicData(NumStreamedMips, Settings, &Packets[PacketIndex].LocalDeferredTickCBAssets); // We always use the Deferred CBs when doing the ParallelFor since those CBs are not thread safe.

				// Make a list of each texture/mesh that can potentially require additional UpdateStreamingStatus
				if (StreamingRenderAsset.RequestedMips != StreamingRenderAsset.ResidentMips)
				{
					Packets[PacketIndex].LocalInflightRenderAssets.Add(Index);
				}

#if STATS
				if (StreamingRenderAsset.ResidentMips > PreviousResidentMips)
				{
					Packets[PacketIndex].MipIOBandwidth += StreamingRenderAsset.GetSize(StreamingRenderAsset.ResidentMips) - StreamingRenderAsset.GetSize(PreviousResidentMips);
				}
#endif
			}
		}, EParallelForFlags::BackgroundPriority);

		for (const FPacket& Packet : Packets)
		{
			InflightRenderAssets.Append(Packet.LocalInflightRenderAssets);
			DeferredTickCBAssets.Append(Packet.LocalDeferredTickCBAssets);
			STAT(GatheredStats.MipIOBandwidth += Packet.MipIOBandwidth;)
		}
	}
	else
	{
		for (int32 Index = StartIndex; Index < EndIndex; ++Index)
		{
			FStreamingRenderAsset& StreamingRenderAsset = AsyncUnsafeStreamingRenderAssets[Index];
			FPlatformMisc::Prefetch(&StreamingRenderAsset + 1);

			// Is this texture/mesh marked for removal? Will get cleanup once the async task is done.
			if (!StreamingRenderAsset.RenderAsset) continue;

			STAT(int32 PreviousResidentMips = StreamingRenderAsset.ResidentMips;)

			const TConstArrayView<int32> NumStreamedMips = GetNumStreamedMipsArray(StreamingRenderAsset.RenderAssetType);

			StreamingRenderAsset.UpdateDynamicData(NumStreamedMips, Settings, bAsync ? &DeferredTickCBAssets : nullptr);

			// Make a list of each texture/mesh that can potentially require additional UpdateStreamingStatus
			if (StreamingRenderAsset.RequestedMips != StreamingRenderAsset.ResidentMips)
			{
				InflightRenderAssets.Add(Index);
			}

#if STATS
			if (StreamingRenderAsset.ResidentMips > PreviousResidentMips)
			{
				GatheredStats.MipIOBandwidth += StreamingRenderAsset.GetSize(StreamingRenderAsset.ResidentMips) - StreamingRenderAsset.GetSize(PreviousResidentMips);
			}
#endif
		}
	}
	CurrentUpdateStreamingRenderAssetIndex = EndIndex;
}

static TAutoConsoleVariable<int32> CVarTextureStreamingAmortizeCPUToGPUCopy(
	TEXT("r.Streaming.AmortizeCPUToGPUCopy"),
	0,
	TEXT("If set and r.Streaming.MaxNumTexturesToStreamPerFrame > 0, limit the number of 2D textures ")
	TEXT("streamed from CPU memory to GPU memory each frame"),
	ECVF_Scalability | ECVF_ExcludeFromPreview);

static TAutoConsoleVariable<int32> CVarTextureStreamingMaxNumTexturesToStreamPerFrame(
	TEXT("r.Streaming.MaxNumTexturesToStreamPerFrame"),
	0,
	TEXT("Maximum number of 2D textures allowed to stream from CPU memory to GPU memory each frame. ")
	TEXT("<= 0 means no limit. This has no effect if r.Streaming.AmortizeCPUToGPUCopy is not set"),
	ECVF_Scalability | ECVF_ExcludeFromPreview);

static FORCEINLINE bool ShouldAmortizeMipCopies()
{
	return CVarTextureStreamingAmortizeCPUToGPUCopy.GetValueOnGameThread()
		&& CVarTextureStreamingMaxNumTexturesToStreamPerFrame.GetValueOnGameThread() > 0;
}

/**
 * Stream textures/meshes in/out, based on the priorities calculated by the async work.
 * @param bProcessEverything	Whether we're processing all textures in one go
 */
void FRenderAssetStreamingManager::StreamRenderAssets( bool bProcessEverything )
{
	TArray<FStreamingRenderAsset>& StreamingRenderAssets = GetStreamingRenderAssetsAsyncSafe();
	const FRenderAssetStreamingMipCalcTask& AsyncTask = AsyncWork->GetTask();

	// Note that render asset indices referred by the async task could be outdated if UpdatePendingStates() was called between the
	// end of the async task work, and this call to StreamRenderAssets(). This happens when SyncStates(false) is called.

	if (!bPauseRenderAssetStreaming || bProcessEverything)
	{
		for (int32 AssetIndex : AsyncTask.GetCancelationRequests())
		{
			if (StreamingRenderAssets.IsValidIndex(AssetIndex) && !VisibleFastResponseRenderAssetIndices.Contains(AssetIndex))
			{
				StreamingRenderAssets[AssetIndex].CancelStreamingRequest();
			}
		}

		if (!bProcessEverything && ShouldAmortizeMipCopies())
		{
			// Ignore remaining requests since they may be outdated already
			PendingMipCopyRequests.Reset();
			CurrentPendingMipCopyRequestIdx = 0;

			// Make copies of the requests so that they can be processed later
			for (int32 AssetIndex : AsyncTask.GetLoadRequests())
			{
				if (StreamingRenderAssets.IsValidIndex(AssetIndex)
					&& StreamingRenderAssets[AssetIndex].RenderAsset
					&& !VisibleFastResponseRenderAssetIndices.Contains(AssetIndex))
				{
					FStreamingRenderAsset& StreamingRenderAsset = StreamingRenderAssets[AssetIndex];
					StreamingRenderAsset.CacheStreamingMetaData();
					new (PendingMipCopyRequests) FPendingMipCopyRequest(StreamingRenderAsset.RenderAsset, AssetIndex);
				}
			}
		}
		else
		{
			for (int32 AssetIndex : AsyncTask.GetLoadRequests())
			{
				if (StreamingRenderAssets.IsValidIndex(AssetIndex) && !VisibleFastResponseRenderAssetIndices.Contains(AssetIndex))
				{
					StreamingRenderAssets[AssetIndex].StreamWantedMips(*this);
				}
			}
		}
	}
	
	for (int32 AssetIndex : AsyncTask.GetPendingUpdateDirties())
	{
		if (StreamingRenderAssets.IsValidIndex(AssetIndex) && !VisibleFastResponseRenderAssetIndices.Contains(AssetIndex))
		{
			FStreamingRenderAsset& StreamingRenderAsset = StreamingRenderAssets[AssetIndex];
			const bool bNewState = StreamingRenderAsset.HasUpdatePending(bPauseRenderAssetStreaming, AsyncTask.HasAnyView());

			// Always update the texture/mesh and the streaming texture/mesh together to make sure they are in sync.
			StreamingRenderAsset.SetFlag(FStreamingRenderAssetFlags::HasUpdatePending, bNewState);
			if (StreamingRenderAsset.RenderAsset)
			{
				StreamingRenderAsset.RenderAsset->bHasStreamingUpdatePending = bNewState;
			}
		}
	}

	// Reset BudgetMipBias and MaxAllowedMips before we forget. Otherwise, new requests may stream out forced mips
	for (TSet<int32>::TConstIterator It(VisibleFastResponseRenderAssetIndices); It; ++It)
	{
		const int32 Idx = *It;

		if (StreamingRenderAssets.IsValidIndex(Idx))
		{
			FStreamingRenderAsset& Asset = StreamingRenderAssets[Idx];
			Asset.BudgetMipBias = 0;
			Asset.SetFlag(FStreamingRenderAssetFlags::IgnoreStreamingMipBias, true);

			// If optional mips is available but not counted, they will be counted later in UpdateDynamicData
			if (Asset.RenderAsset)
			{
				Asset.MaxAllowedMips = FMath::Max<int32>(Asset.MaxAllowedMips, Asset.RenderAsset->GetStreamableResourceState().NumNonOptionalLODs);
			}
		}
	}

	VisibleFastResponseRenderAssetIndices.Empty();
}

void FRenderAssetStreamingManager::ProcessPendingMipCopyRequests()
{
	if (!ShouldAmortizeMipCopies())
	{
		return;
	}

	TArray<FStreamingRenderAsset>& StreamingRenderAssets = GetStreamingRenderAssetsAsyncSafe();
	int32 NumRemainingRequests = CVarTextureStreamingMaxNumTexturesToStreamPerFrame.GetValueOnGameThread();

	while (NumRemainingRequests
		&& CurrentPendingMipCopyRequestIdx < PendingMipCopyRequests.Num())
	{
		const FPendingMipCopyRequest& Request = PendingMipCopyRequests[CurrentPendingMipCopyRequestIdx++];

		if (Request.RenderAsset)
		{
			FStreamingRenderAsset* StreamingRenderAsset = nullptr;

			if (StreamingRenderAssets.IsValidIndex(Request.CachedIdx)
				&& StreamingRenderAssets[Request.CachedIdx].RenderAsset == Request.RenderAsset)
			{
				StreamingRenderAsset = &StreamingRenderAssets[Request.CachedIdx];
			}
			else if (ReferencedRenderAssets.Contains(Request.RenderAsset))
			{
				// Texture is still valid but its index has been changed
				check(StreamingRenderAssets.IsValidIndex(Request.RenderAsset->StreamingIndex));
				StreamingRenderAsset = &StreamingRenderAssets[Request.RenderAsset->StreamingIndex];
			}

			if (StreamingRenderAsset)
			{
				StreamingRenderAsset->StreamWantedMipsUsingCachedData(*this);
				--NumRemainingRequests;
			}
		}
	}
}

void FRenderAssetStreamingManager::TickDeferredMipLevelChangeCallbacks()
{
	if (DeferredTickCBAssets.Num() > 0)
	{
		check(IsInGameThread());

		if(StreamingRenderAssetsSyncEvent.IsValid())
		{
			StreamingRenderAssetsSyncEvent->Wait(ENamedThreads::GameThread);
		}

		for (int32 AssetIdx = 0; AssetIdx < DeferredTickCBAssets.Num(); ++AssetIdx)
		{
			UStreamableRenderAsset* Asset = DeferredTickCBAssets[AssetIdx];
			if (Asset)
			{
				Asset->TickMipLevelChangeCallbacks(nullptr);
			}
		}
		DeferredTickCBAssets.Empty();
	}
}

void FRenderAssetStreamingManager::CheckUserSettings()
{	
	if (CVarStreamingUseFixedPoolSize.GetValueOnGameThread() == 0)
	{
		const int32 PoolSizeSetting = CVarStreamingPoolSize.GetValueOnGameThread();

		int64 TexturePoolSize = GTexturePoolSize;
		if (PoolSizeSetting == -1)
		{
			FTextureMemoryStats Stats;
			RHIGetTextureMemoryStats(Stats);
			if (GPoolSizeVRAMPercentage > 0 && Stats.TotalGraphicsMemory > 0)
			{
				TexturePoolSize = Stats.TotalGraphicsMemory * GPoolSizeVRAMPercentage / 100;
			}
		}
		else
		{
			TexturePoolSize = int64(PoolSizeSetting) * 1024ll * 1024ll;
		}

		if (TexturePoolSize != GTexturePoolSize)
		{
			UE_LOGF(LogContentStreaming,Log,"Texture pool size now %d MB", int32(TexturePoolSize/1024/1024));
			CSV_METADATA(TEXT("StreamingPoolSizeMB"), *WriteToString<32>(int32(TexturePoolSize / 1024 / 1024)));
			GTexturePoolSize = TexturePoolSize;
		}
	}
}

void FRenderAssetStreamingManager::SetLastUpdateTime()
{
	// Update the last update time.
	float WorldTime = 0;

	if (FSimpleStreamableAssetManager::IsEnabled())
	{
		if (FSimpleStreamableAssetManager::ShouldConsiderVisibility())
		{
			for (const FWorldContext& Context : GEngine->GetWorldContexts())
			{
				WorldTime = FRenderAssetStreamingManagerHelper::GetWorldTime(Context.World());
				if (WorldTime > 0)
				{
					break;
				}
			}
		}
	}
	else
	{
		for (FLevelRenderAssetManager* LevelRenderAssetManager : LevelRenderAssetManagers)
		{
			// Update last update time only if there is a reasonable threshold to define visibility.
			WorldTime = LevelRenderAssetManager ? LevelRenderAssetManager->GetWorldTime() : 0.f;
			if (WorldTime > 0)
			{
				break;
			}
		}
	}

	if (WorldTime > 0)
	{
		LastWorldUpdateTime = WorldTime - .5f;
	}
	else if (GIsEditor)
	{
		LastWorldUpdateTime = -FLT_MAX; // In editor, visibility is not taken into consideration unless in PIE.
	}
}

void FRenderAssetStreamingManager::UpdateStats()
{
	float DeltaStatTime = (float)(GatheredStats.Timestamp - DisplayedStats.Timestamp);
	if (DeltaStatTime > UE_SMALL_NUMBER)
	{
		GatheredStats.MipIOBandwidth = DeltaStatTime > UE_SMALL_NUMBER ? GatheredStats.MipIOBandwidth / DeltaStatTime : 0;
	}
	DisplayedStats = GatheredStats;
	GatheredStats.CallbacksCycles = 0;
	GatheredStats.MipIOBandwidth = 0;
	MemoryOverBudget = DisplayedStats.OverBudget;
	MaxEverRequired = FMath::Max<int64>(MaxEverRequired, DisplayedStats.RequiredPool);
}

void FRenderAssetStreamingManager::UpdateCSVOnlyStats()
{
	DisplayedStats = GatheredStats;
}

void FRenderAssetStreamingManager::LogViewLocationChange()
{
#if STREAMING_LOG_VIEWCHANGES
	static bool bWasLocationOveridden = false;
	bool bIsLocationOverridden = false;
	for ( int32 ViewIndex=0; ViewIndex < CurrentViewInfos.Num(); ++ViewIndex )
	{
		FStreamingViewInfo& ViewInfo = CurrentViewInfos[ViewIndex];
		if ( ViewInfo.bOverrideLocation )
		{
			bIsLocationOverridden = true;
			break;
		}
	}
	if ( bIsLocationOverridden != bWasLocationOveridden )
	{
		UE_LOGF(LogContentStreaming, Log, "Texture streaming view location is now %ls.", bIsLocationOverridden ? TEXT("OVERRIDDEN") : TEXT("normal") );
		bWasLocationOveridden = bIsLocationOverridden;
	}
#endif
}

/**
 * Main function for the texture streaming system, based on texture priorities and asynchronous processing.
 * Updates streaming, taking into account all view infos.
 *
 * @param DeltaTime				Time since last call in seconds
 * @param bProcessEverything	[opt] If true, process all resources with no throttling limits
 */

static TAutoConsoleVariable<int32> CVarUseBackgroundThreadPool(
	TEXT("r.Streaming.UseBackgroundThreadPool"),
	1,
	TEXT("If true, use the background thread pool for mip calculations."));

class FUpdateStreamingRenderAssetsTask
{
	FRenderAssetStreamingManager* Manager;
	int32 StageIdx;
	int32 NumUpdateStages;
public:
	FUpdateStreamingRenderAssetsTask(FRenderAssetStreamingManager* InManager, int32 InStageIdx, int32 InNumUpdateStages)
		: Manager(InManager)
		, StageIdx(InStageIdx)
		, NumUpdateStages(InNumUpdateStages)
	{
	}
	static FORCEINLINE TStatId GetStatId()
	{
		RETURN_QUICK_DECLARE_CYCLE_STAT(FUpdateStreamingRenderAssetsTask, STATGROUP_TaskGraphTasks);
	}
	static FORCEINLINE ENamedThreads::Type GetDesiredThread()
	{
		return ENamedThreads::AnyNormalThreadNormalTask;
	}
	static FORCEINLINE ESubsequentsMode::Type GetSubsequentsMode()
	{
		return ESubsequentsMode::TrackSubsequents;
	}
	void DoTask(ENamedThreads::Type CurrentThread, const FGraphEventRef& MyCompletionGraphEvent)
	{
		FTaskTagScope Scope(ETaskTag::EParallelGameThread);
		Manager->UpdateStreamingRenderAssets(StageIdx, NumUpdateStages, true);
	}
};

void FRenderAssetStreamingManager::UpdateResourceStreaming( float DeltaTime, bool bProcessEverything/*=false*/ )
{
	UE::TScopeLock ScopeLock(CriticalSection);
	TArray<FStreamingRenderAsset>& StreamingRenderAssets = GetStreamingRenderAssetsAsyncSafe();

	SCOPE_CYCLE_COUNTER(STAT_RenderAssetStreaming_GameThreadUpdateTime);
	CSV_SCOPED_TIMING_STAT_EXCLUSIVE(RenderAssetStreaming);
	CSV_SCOPED_SET_WAIT_STAT(RenderAssetStreaming);

	const bool bUseThreadingForPerf = FApp::ShouldUseThreadingForPerformance();

	LogViewLocationChange();
	STAT(DisplayedStats.Apply();)

	float StreamingPoolStat = ((float)(DisplayedStats.RequiredPool + (GPoolSizeVRAMPercentage > 0 ? 0 : DisplayedStats.NonStreamingMips)));
	CSV_CUSTOM_STAT(TextureStreaming, StreamingPool, StreamingPoolStat / (1024.0f * 1024.0f), ECsvCustomStatOp::Set);
	CSV_CUSTOM_STAT(TextureStreaming, SafetyPool, ((float)DisplayedStats.SafetyPool) / (1024.0f * 1024.0f), ECsvCustomStatOp::Set);
	CSV_CUSTOM_STAT(TextureStreaming, TemporaryPool, ((float)DisplayedStats.TemporaryPool) / (1024.0f * 1024.0f), ECsvCustomStatOp::Set);
	CSV_CUSTOM_STAT(TextureStreaming, CachedMips, ((float)DisplayedStats.CachedMips) / (1024.0f * 1024.0f), ECsvCustomStatOp::Set);
	CSV_CUSTOM_STAT(TextureStreaming, WantedMips, ((float)DisplayedStats.WantedMips) / (1024.0f * 1024.0f), ECsvCustomStatOp::Set);
	CSV_CUSTOM_STAT(TextureStreaming, ResidentMeshMem, ((float)DisplayedStats.ResidentMeshMem) / (1024.0f * 1024.0f), ECsvCustomStatOp::Set);
	CSV_CUSTOM_STAT(TextureStreaming, StreamedMeshMem, ((float)DisplayedStats.StreamedMeshMem) / (1024.0f * 1024.0f), ECsvCustomStatOp::Set);
	CSV_CUSTOM_STAT(TextureStreaming, NonStreamingMips, ((float)DisplayedStats.NonStreamingMips) / (1024.0f * 1024.0f), ECsvCustomStatOp::Set);
	CSV_CUSTOM_STAT(TextureStreaming, PendingStreamInData, ((float)DisplayedStats.PendingRequests) / (1024.0f * 1024.0f), ECsvCustomStatOp::Set);

	float DesiredDataLoadedPct = StreamingPoolStat == 0.0f ? 0.0f : 100.0f * (float)DisplayedStats.WantedMips / StreamingPoolStat;
	CSV_CUSTOM_STAT(TextureStreaming, DesiredDataLoadedPercent, DesiredDataLoadedPct, ECsvCustomStatOp::Set);

	RenderAssetInstanceAsyncWork->EnsureCompletion();

	if (NumRenderAssetProcessingStages <= 0 || bProcessEverything)
	{
		if (!AsyncWork->IsDone())
		{	// Is the AsyncWork is running for some reason? (E.g. we reset the system by simply setting ProcessingStage to 0.)
			AsyncWork->EnsureCompletion();
		}

		ProcessingStage = 0;
		NumRenderAssetProcessingStages = Settings.FramesForFullUpdate;

		// Update Thread Data
		SetLastUpdateTime();
		UpdateStreamingRenderAssets(0, 1);

		UpdatePendingStates(true);
		PrepareAsyncTask(bProcessEverything || Settings.bStressTest);
		AsyncWork->StartSynchronousTask();

		TickFastResponseAssets();

		StreamRenderAssets(bProcessEverything);

		// Release the old view now as the destructors can be expensive. Now only the dynamic manager holds a ref.
		AsyncWork->GetTask().ReleaseAsyncViews();

		STAT(GatheredStats.SetupAsyncTaskCycles = 0);
		STAT(GatheredStats.UpdateStreamingDataCycles = 0);
		STAT(GatheredStats.StreamRenderAssetsCycles = 0);
		STAT(GatheredStats.CallbacksCycles = 0);
#if STATS
		UpdateStats();
#elif UE_WITH_RENDER_ASSET_STREAMING_CSV_STATS
		UpdateCSVOnlyStats();
#endif // STATS
	}
	else if (ProcessingStage == 0)
	{
		STAT(GatheredStats.SetupAsyncTaskCycles = -(int32)FPlatformTime::Cycles();)

		NumRenderAssetProcessingStages = Settings.FramesForFullUpdate;

		if (!AsyncWork->IsDone())
		{	// Is the AsyncWork is running for some reason? (E.g. we reset the system by simply setting ProcessingStage to 0.)
			AsyncWork->EnsureCompletion();
		}

		// Here we rely on dynamic components to be updated on the last stage, in order to split the workload. 
		UpdatePendingStates(false);
		PrepareAsyncTask(bProcessEverything || Settings.bStressTest);
		AsyncWork->StartBackgroundTask(CVarUseBackgroundThreadPool.GetValueOnGameThread() ? GBackgroundPriorityThreadPool : GThreadPool);
		TickFastResponseAssets();
		++ProcessingStage;

		STAT(GatheredStats.SetupAsyncTaskCycles += FPlatformTime::Cycles();)
	}
	else if (ProcessingStage <= NumRenderAssetProcessingStages)
	{
		STAT(int32 StartTime = (int32)FPlatformTime::Cycles();)

		if (PendingStreamingRenderAssets.Num() > 0 && CVarProcessAddedRenderAssetsAfterAsyncWork.GetValueOnGameThread() && AsyncWork->IsDone())
		{
			// This will add to the StreamingRenderAssets array potentially reallocating it, but if the Async task has completed, that should be safe.
			// As we're only adding items, existing indicies in InflightRenderAssets etc will still be valid.
			ProcessAddedRenderAssets();
		}

		if (ProcessingStage == 1)
		{
			SetLastUpdateTime();
		}

		TickFastResponseAssets();

		// Optimization: overlapping UpdateStreamingRenderAssets() and IncrementalUpdate();
		// Restrict this optimization to platforms tested to have a win;
		// Platforms tested and results (ave exec time of UpdateResourceStreaming):
		//   PS4 Pro - from ~0.55 ms/frame to ~0.15 ms/frame
		//   XB1 X - from ~0.45 ms/frame to ~0.17 ms/frame
		const bool bOverlappedExecution = bUseThreadingForPerf && CVarStreamingOverlapAssetAndLevelTicks.GetValueOnGameThread();
		if (bOverlappedExecution)
		{
			if (StreamingRenderAssetsSyncEvent.IsValid() && StreamingRenderAssetsSyncEvent->IsComplete())
			{
				StreamingRenderAssetsSyncEvent = nullptr;
			}

			StreamingRenderAssetsSyncEvent = TGraphTask<FUpdateStreamingRenderAssetsTask>::CreateTask(nullptr, ENamedThreads::GameThread)
			.ConstructAndDispatchWhenReady(this, ProcessingStage - 1, NumRenderAssetProcessingStages);
		}
		else
		{
			UpdateStreamingRenderAssets(ProcessingStage - 1, NumRenderAssetProcessingStages);
		}

		IncrementalUpdate(1.f / (float)FMath::Max(NumRenderAssetProcessingStages - 1, 1), true); // -1 since we don't want to do anything at stage 0.
		++ProcessingStage;

		STAT(GatheredStats.UpdateStreamingDataCycles = FMath::Max<uint32>(ProcessingStage > 2 ? GatheredStats.UpdateStreamingDataCycles : 0, FPlatformTime::Cycles() - StartTime);)
	}
	else if (AsyncWork->IsDone())
	{
		STAT(GatheredStats.StreamRenderAssetsCycles = -(int32)FPlatformTime::Cycles();)

		if (PendingStreamingRenderAssets.Num() > 0 && CVarProcessAddedRenderAssetsAfterAsyncWork.GetValueOnGameThread())
		{
			// This will add to the StreamingRenderAssets array potentially reallocating it, but if the Async task has completed, that should be safe.
			// As we're only adding items, existing indicies in InflightRenderAssets etc will still be valid.
			ProcessAddedRenderAssets();
		}

		// Since this step is lightweight, tick each texture inflight here, to accelerate the state changes.
		for (int32 TextureIndex : InflightRenderAssets)
		{
			StreamingRenderAssets[TextureIndex].UpdateStreamingStatus();
		}

		TickFastResponseAssets();

		StreamRenderAssets(bProcessEverything);
		// Release the old view now as the destructors can be expensive. Now only the dynamic manager holds a ref.
		AsyncWork->GetTask().ReleaseAsyncViews();
		IncrementalUpdate(1.f / (float)FMath::Max(NumRenderAssetProcessingStages - 1, 1), true); // Just in case continue any pending update.

		if (!FSimpleStreamableAssetManager::IsEnabled())
		{
			DynamicComponentManager.PrepareAsyncView();
		}
		
		ProcessingStage = 0;

		STAT(GatheredStats.StreamRenderAssetsCycles += FPlatformTime::Cycles();)
#if STATS
		UpdateStats();
#elif UE_WITH_RENDER_ASSET_STREAMING_CSV_STATS
		UpdateCSVOnlyStats();
#endif // STATS
	}

	if (!bProcessEverything)
	{
		ProcessPendingMipCopyRequests();
	}

	TickDeferredMipLevelChangeCallbacks();

	if (GEnableAssetUpdateAbandons && GTickAbandonedRenderAssetUpdatesOnStreamingUpdate)
	{
		AbandonedRenderAssetUpdateManager.TickAbandoned();
	}

	if (bUseThreadingForPerf)
	{
		RenderAssetInstanceAsyncWork->StartBackgroundTask(GThreadPool);
	}
	else
	{
		RenderAssetInstanceAsyncWork->StartSynchronousTask();
	}
}

/**
 * Blocks till all pending requests are fulfilled.
 *
 * @param TimeLimit		Optional time limit for processing, in seconds. Specifying 0 means infinite time limit.
 * @param bLogResults	Whether to dump the results to the log.
 * @return				Number of streaming requests still in flight, if the time limit was reached before they were finished.
 */
int32 FRenderAssetStreamingManager::BlockTillAllRequestsFinished( float TimeLimit /*= 0.0f*/, bool bLogResults /*= false*/ )
{
	UE::TScopeLock ScopeLock(CriticalSection);
	TArray<FStreamingRenderAsset>& StreamingRenderAssets = GetStreamingRenderAssetsAsyncSafe();
	QUICK_SCOPE_CYCLE_COUNTER(STAT_FRenderAssetStreamingManager_BlockTillAllRequestsFinished);

	double StartTime = FPlatformTime::Seconds();

	while (ensure(!IsAssetStreamingSuspended()))
	{
		// Optionally synchronize the states of async work before we wait for outstanding work to be completed.
		if (CVarSyncStatesWhenBlocking.GetValueOnGameThread() != 0)
		{
			SyncStates(true);
		}

		int32 NumOfInFlights = 0;

		for (FStreamingRenderAsset& StreamingRenderAsset : StreamingRenderAssets)
		{
			const bool bValid = StreamingRenderAsset.UpdateStreamingStatus().IsValid();
			if (bValid && StreamingRenderAsset.RequestedMips != StreamingRenderAsset.ResidentMips)
			{
				++NumOfInFlights;
			}
		}

		if (NumOfInFlights && (TimeLimit == 0 || (float)(FPlatformTime::Seconds() - StartTime) < TimeLimit))
		{
			FlushRenderingCommands();
			FPlatformProcess::Sleep(RENDER_ASSET_STREAMING_SLEEP_DT);
		}
		else
		{
			if (bLogResults)
			{
				float BlockedMillis = (float)(FPlatformTime::Seconds() - StartTime) * 1000;
				if ( BlockedMillis > 0.1f )
				{
					UE_LOGF(LogContentStreaming, Log, "Blocking on texture streaming: %.1f ms (%d still in flight)", BlockedMillis, NumOfInFlights);
				}
			}
			return NumOfInFlights;
		}
	}
	return 0;
}

void FRenderAssetStreamingManager::GetObjectReferenceBounds(const UObject* RefObject, TArray<FBox>& AssetBoxes)
{
	UE::TScopeLock ScopeLock(CriticalSection);
	QUICK_SCOPE_CYCLE_COUNTER(STAT_FRenderAssetStreamingManager_GetObjectReferenceBounds);

	const UStreamableRenderAsset* RenderAsset = Cast<const UStreamableRenderAsset>(RefObject);
	if (RenderAsset && !FSimpleStreamableAssetManager::IsEnabled())
	{
		for (FLevelRenderAssetManager *LevelManager : LevelRenderAssetManagers)
		{
			if (LevelManager == nullptr)
			{
				continue;
			}

			const FRenderAssetInstanceView* View = LevelManager->GetRawAsyncView();
			if (View)
			{
				for (auto It = View->GetElementIterator(RenderAsset); It; ++It)
				{
					AssetBoxes.Add(It.GetBounds().GetBox());
				}
			}
		}

		const FRenderAssetInstanceView* View = DynamicComponentManager.GetAsyncView(false);
		if (View)
		{
			for (auto It = View->GetElementIterator(RenderAsset); It; ++It)
			{
				AssetBoxes.Add(It.GetBounds().GetBox());
			}
		}
	}

	if (RenderAsset && FSimpleStreamableAssetManager::IsEnabled())
	{
		FSimpleStreamableAssetManager::GetAssetReferenceBounds(RenderAsset, AssetBoxes);
	}
}

static bool IsLevelManagerValid(const FLevelRenderAssetManager* LevelManager)
{
	return LevelManager && LevelManager->IsInitialized() && LevelManager->HasRenderAssetReferences() && LevelManager->GetLevel()->bIsVisible;
}

static void GatherComponentsFromView(
	const UStreamableRenderAsset* RenderAsset,
	const FRenderAssetInstanceView* View,
	const TFunction<bool(const IPrimitiveComponent*)>& ShouldChoose,
	TArray<const IPrimitiveComponent*>& OutComps)
{
	if (View)
	{
		for (FRenderAssetInstanceView::FRenderAssetLinkConstIterator It = View->GetElementIterator(RenderAsset); It; ++It)
		{
			const IPrimitiveComponent* Comp = It.GetComponent();
			if (Comp && ShouldChoose(Comp))
			{
				OutComps.Add(Comp);
			}
		}
	}
}

void FRenderAssetStreamingManager::GetAssetComponents(const UStreamableRenderAsset* RenderAsset, TArray<const IPrimitiveComponent*>& OutComps, TFunction<bool(const IPrimitiveComponent*)> ShouldChoose)
{
	checkSlow(IsInGameThread());
	
	if (FSimpleStreamableAssetManager::IsEnabled())
	{
		return;
	}
	
	if (RenderAsset && RenderAsset->StreamingIndex >= 0)
	{
		UE::TScopeLock Lock(CriticalSection);
		QUICK_SCOPE_CYCLE_COUNTER(STAT_FRenderAssetStreamingManager_GetAssetComponents);

		const int32 Idx = RenderAsset->StreamingIndex;
		check(AsyncUnsafeStreamingRenderAssets.IsValidIndex(Idx) && AsyncUnsafeStreamingRenderAssets[Idx].RenderAsset == RenderAsset);
		const FStreamingRenderAsset& StreamingRenderAsset = AsyncUnsafeStreamingRenderAssets[Idx];

		for (TConstSetBitIterator<> It(StreamingRenderAsset.LevelIndexUsage); It; ++It)
		{
			const int32 LevelIdx = It.GetIndex();
			check(LevelRenderAssetManagers.IsValidIndex(LevelIdx));
			FLevelRenderAssetManager* LevelManager = LevelRenderAssetManagers[LevelIdx];

			if (IsLevelManagerValid(LevelManager))
			{
				const FRenderAssetInstanceView* StaticInstancesView = LevelManager->GetRawAsyncView();
				GatherComponentsFromView(RenderAsset, StaticInstancesView, ShouldChoose, OutComps);
			}
		}

		const FRenderAssetInstanceView* DynamicInstancesView = DynamicComponentManager.GetGameThreadView();
		GatherComponentsFromView(RenderAsset, DynamicInstancesView, ShouldChoose, OutComps);
	}
}

void FRenderAssetStreamingManager::PropagateLightingScenarioChange()
{
	if (FSimpleStreamableAssetManager::IsEnabled())
	{
		return;
	}
	
	UE::TScopeLock ScopeLock(CriticalSection);
	QUICK_SCOPE_CYCLE_COUNTER(STAT_FRenderAssetStreamingManager_PropagateLightingScenarioChange);

	// Note that dynamic components don't need to be handled because their renderstates are updated, which triggers and update.
	
	TArray<ULevel*, TInlineAllocator<32> > Levels;
	check(!LevelRenderAssetManagersLock);
	for (FLevelRenderAssetManager* LevelManager : LevelRenderAssetManagers)
	{
		if (LevelManager!=nullptr)
		{
			Levels.Push(LevelManager->GetLevel());
			LevelManager->Remove(nullptr);
            delete LevelManager;
		}
	}

	LevelRenderAssetManagers.Empty();

	for (ULevel* Level : Levels)
	{
		AddLevel(Level);
	}
}

void FRenderAssetStreamingManager::AddRenderedTextureStats(TMap<FString, FRenderedTextureStats>& OutRenderedTextureStats)
{
	UE::TScopeLock ScopeLock(CriticalSection);
	
	for (const FStreamingRenderAsset& StreamingRenderAsset : AsyncUnsafeStreamingRenderAssets)
	{
		const UStreamableRenderAsset* RenderAsset = StreamingRenderAsset.RenderAsset;
		if (RenderAsset == nullptr || StreamingRenderAsset.GetFlag(FStreamingRenderAssetFlags::UseUnkownRefHeuristic)
			|| StreamingRenderAsset.RenderAssetType != EStreamableRenderAssetType::Texture || StreamingRenderAsset.LastRenderTime == UE_MAX_FLT)
		{
			continue;
		}

		const int32 MipArrayIndex = FMath::Max(0, StreamingRenderAsset.MaxAllowedMips - StreamingRenderAsset.ResidentMips);
		FRenderedTextureStats* Stats = OutRenderedTextureStats.Find(RenderAsset->GetName());
		if(Stats != nullptr) 
		{
			// Keeps max mip level ever shown
			Stats->MaxMipLevelShown = MipArrayIndex < Stats->MaxMipLevelShown ? MipArrayIndex : Stats->MaxMipLevelShown;
		} 
		else 
		{
			FRenderedTextureStats NewStats;
			NewStats.MaxMipLevelShown = MipArrayIndex;
			NewStats.TextureGroup = UTexture::GetTextureGroupString(static_cast<TextureGroup>(StreamingRenderAsset.LODGroup));
			OutRenderedTextureStats.Add(RenderAsset->GetName(), NewStats);
		}
	}
}

void FRenderAssetStreamingManager::OnRenderAssetUpdateAbandoned(UStreamableRenderAsset* OwningRenderAsset, TRefCountPtr<class FRenderAssetUpdate>& RenderAssetUpdate)
{
	check(GEnableAssetUpdateAbandons);
	AbandonedRenderAssetUpdateManager.OnAbandoned(OwningRenderAsset, RenderAssetUpdate);
}

#if !UE_BUILD_SHIPPING

bool FRenderAssetStreamingManager::HandleDumpTextureStreamingStatsCommand( const TCHAR* Cmd, FOutputDevice& Ar )
{
	UE::TScopeLock ScopeLock(CriticalSection);

	float CurrentOccupancyPct = 0.f;
	float TargetOccupancyPct = 0.f;
	if (DisplayedStats.StreamingPool > 0)
	{
		CurrentOccupancyPct = 100.f * (DisplayedStats.WantedMips / (float)DisplayedStats.StreamingPool);
		TargetOccupancyPct = 100.f * (DisplayedStats.RequiredPool / (float)DisplayedStats.StreamingPool);
	}

	float StreamingPoolMB = DisplayedStats.StreamingPool / 1024.f / 1024.f;

	// uses csv stats for access in test builds
	Ar.Logf(TEXT("--------------------------------------------------------"));
	Ar.Logf(TEXT("Texture Streaming Stats:") );
	Ar.Logf(TEXT("Total Pool Size (aka RenderAssetPool) = %.2f MB"), DisplayedStats.RenderAssetPool / 1024.f / 1024.f);
	Ar.Logf(TEXT("Non-Streaming Mips = %.2f MB"), DisplayedStats.NonStreamingMips / 1024.f / 1024.f);
	Ar.Logf(TEXT("Remaining Streaming Pool Size = %.2f MB"), StreamingPoolMB);
	Ar.Logf(TEXT("Streaming Assets, Current/Pool = %.2f / %.2f MB (%d%%)"), DisplayedStats.WantedMips / 1024.f / 1024.f, StreamingPoolMB, FMath::RoundToInt(CurrentOccupancyPct));
	Ar.Logf(TEXT("Streaming Assets, Target/Pool =  %.2f / %.2f MB (%d%%)"), DisplayedStats.RequiredPool / 1024.f / 1024.f, StreamingPoolMB, FMath::RoundToInt(TargetOccupancyPct));
	Ar.Logf(TEXT("--------------------------------------------------------"));

	return true;
}

bool FRenderAssetStreamingManager::HandleListStreamingRenderAssetsCommand( const TCHAR* Cmd, FOutputDevice& Ar )
{
	UE::TScopeLock ScopeLock(CriticalSection);
	TArray<FStreamingRenderAsset>& StreamingRenderAssets = GetStreamingRenderAssetsAsyncSafe();

	SyncStates(true);

	const bool bShouldOnlyListUnkownRef = FParse::Command(&Cmd, TEXT("UNKOWNREF"));
	EStreamableRenderAssetType ListAssetType = EStreamableRenderAssetType::None;
	{
		FString AssetTypeStr;
		if (FParse::Value(Cmd, TEXT("AssetType="), AssetTypeStr))
		{
			if (AssetTypeStr == TEXT("Texture"))
			{
				ListAssetType = EStreamableRenderAssetType::Texture;
			}
			else if (AssetTypeStr == TEXT("StaticMesh"))
			{
				ListAssetType = EStreamableRenderAssetType::StaticMesh;
			}
			else if (AssetTypeStr == TEXT("SkeletalMesh"))
			{
				ListAssetType = EStreamableRenderAssetType::SkeletalMesh;
			}
		}
	}

	// Sort texture/mesh by names so that the state can be compared between runs.
	TMap<FString, int32> SortedRenderAssets;
	for ( int32 Idx=0; Idx < StreamingRenderAssets.Num(); ++Idx)
	{
		const FStreamingRenderAsset& StreamingRenderAsset = StreamingRenderAssets[Idx];
		if (!StreamingRenderAsset.RenderAsset) continue;
		if (bShouldOnlyListUnkownRef && !StreamingRenderAsset.GetFlag(FStreamingRenderAssetFlags::UseUnkownRefHeuristic)) continue;

		SortedRenderAssets.Add(StreamingRenderAsset.RenderAsset->GetFullName(), Idx);
	}

	SortedRenderAssets.KeySort(TLess<FString>());

	for (TMap<FString, int32>::TConstIterator It(SortedRenderAssets); It; ++It)
	{
		const FStreamingRenderAsset& StreamingRenderAsset = StreamingRenderAssets[It.Value()];
		const UStreamableRenderAsset* RenderAsset = StreamingRenderAsset.RenderAsset;
		const FStreamableRenderResourceState ResourceState = RenderAsset->GetStreamableResourceState();
		const EStreamableRenderAssetType AssetType = StreamingRenderAsset.RenderAssetType;
		
		if (ListAssetType != EStreamableRenderAssetType::None && ListAssetType != AssetType)
		{
			continue;
		}

		UE_LOGF(LogContentStreaming, Log, "%ls [%d] : %ls",
			FStreamingRenderAsset::GetStreamingAssetTypeStr(AssetType),
			It.Value(),
			*RenderAsset->GetFullName());

		const int32 CurrentMipIndex = ResourceState.LODCountToAssetFirstLODIdx(ResourceState.NumResidentLODs);
		const int32 WantedMipIndex = ResourceState.LODCountToAssetFirstLODIdx(StreamingRenderAsset.GetPerfectWantedMips());
		const int32 MaxAllowedMipIndex = ResourceState.LODCountToAssetFirstLODIdx(StreamingRenderAsset.MaxAllowedMips);

		FTexturePlatformData** TexturePlatformData = Cast<UTexture>(RenderAsset) ? const_cast<UTexture*>(Cast<UTexture>(RenderAsset))->GetRunningPlatformData() : nullptr;
		if (AssetType == EStreamableRenderAssetType::Texture && TexturePlatformData && *TexturePlatformData)
		{
			const UTexture2D* Texture2D = Cast<UTexture2D>(RenderAsset);
			const UVolumeTexture* VolumeTexture = Cast<UVolumeTexture>(RenderAsset);
			const UTexture2DArray* Texture2DArray = Cast<UTexture2DArray>(RenderAsset);
			const TIndirectArray<struct FTexture2DMipMap>& TextureMips = (*TexturePlatformData)->Mips;

			auto OutputMipsString = [&](int32 OutputIndex)->FString
			{
				const FTexture2DMipMap& OutputMips = TextureMips[OutputIndex];
				if (Texture2D)
				{
					return FString::Printf(TEXT("%dx%d"), OutputMips.SizeX, OutputMips.SizeY);
				}
				else if (VolumeTexture)
				{
					return FString::Printf(TEXT("%dx%dx%d"), OutputMips.SizeX, OutputMips.SizeY, OutputMips.SizeZ);
				}
				else if (Texture2DArray)
				{
					return FString::Printf(TEXT("%dx%d*%d"), OutputMips.SizeX, OutputMips.SizeY, OutputMips.SizeZ);
				}
				else // Unkown type fallback
				{
					return FString::Printf(TEXT("%d?%d?%d"), OutputMips.SizeX, OutputMips.SizeY, OutputMips.SizeZ);
				}
			};

			if (StreamingRenderAsset.LastRenderTime != UE_MAX_FLT)
			{
				UE_LOGF(LogContentStreaming, Log, "    Current=%ls  Wanted=%ls MaxAllowed=%ls LastRenderTime=%.3f BudgetBias=%d Group=%ls",
					*OutputMipsString(CurrentMipIndex), *OutputMipsString(WantedMipIndex), *OutputMipsString(MaxAllowedMipIndex),
					StreamingRenderAsset.LastRenderTime, StreamingRenderAsset.BudgetMipBias,
					UTexture::GetTextureGroupString(static_cast<TextureGroup>(StreamingRenderAsset.LODGroup)));
			}
			else
			{
				UE_LOGF(LogContentStreaming, Log, "    Current=%ls Wanted=%ls MaxAllowed=%ls BudgetBias=%d Group=%ls",
					*OutputMipsString(CurrentMipIndex), *OutputMipsString(WantedMipIndex), *OutputMipsString(MaxAllowedMipIndex),
					StreamingRenderAsset.BudgetMipBias,
					UTexture::GetTextureGroupString(static_cast<TextureGroup>(StreamingRenderAsset.LODGroup)));
			}
		}
		else
		{
			const float LastRenderTime = StreamingRenderAsset.LastRenderTime;
			const UStaticMesh* StaticMesh = Cast<UStaticMesh>(RenderAsset);
			FString LODGroupName = "Unknown";
#if WITH_EDITORONLY_DATA
			if (StaticMesh)
			{
				LODGroupName = StaticMesh->GetLODGroup().ToString();
			}
#endif
			UE_LOGF(LogContentStreaming, Log, "    CurrentLOD=%d WantedLOD=%d MaxAllowedLOD=%d NumLODs=%d NumForcedLODs=%d LastRenderTime=%ls BudgetBias=%d Group=%ls",
				CurrentMipIndex,
				WantedMipIndex,
				MaxAllowedMipIndex,
				ResourceState.MaxNumLODs,
				StreamingRenderAsset.NumForcedMips,
				LastRenderTime == UE_MAX_FLT ? TEXT("NotTracked") : *FString::Printf(TEXT("%.3f"), LastRenderTime),
				StreamingRenderAsset.BudgetMipBias,
				*LODGroupName);
		}
	}
	return true;
}

bool FRenderAssetStreamingManager::HandleResetMaxEverRequiredRenderAssetMemoryCommand(const TCHAR* Cmd, FOutputDevice& Ar)
{
	UE::TScopeLock ScopeLock(CriticalSection);

	Ar.Logf(TEXT("OldMax: %u MaxEverRequired Reset."), MaxEverRequired);
	ResetMaxEverRequired();	
	return true;
}

bool FRenderAssetStreamingManager::HandleLightmapStreamingFactorCommand( const TCHAR* Cmd, FOutputDevice& Ar )
{
	UE::TScopeLock ScopeLock(CriticalSection);

	FString FactorString(FParse::Token(Cmd, 0));
	float NewFactor = ( FactorString.Len() > 0 ) ? FCString::Atof(*FactorString) : GLightmapStreamingFactor;
	if ( NewFactor >= 0.0f )
	{
		GLightmapStreamingFactor = NewFactor;
	}
	Ar.Logf( TEXT("Lightmap streaming factor: %.3f (lower values makes streaming more aggressive)."), GLightmapStreamingFactor );
	return true;
}

bool FRenderAssetStreamingManager::HandleCancelRenderAssetStreamingCommand( const TCHAR* Cmd, FOutputDevice& Ar )
{
	UE::TScopeLock ScopeLock(CriticalSection);

	UTexture::CancelPendingTextureStreaming();
	UStaticMesh::CancelAllPendingStreamingActions();
	USkeletalMesh::CancelAllPendingStreamingActions();
	return true;
}

bool FRenderAssetStreamingManager::HandleShadowmapStreamingFactorCommand( const TCHAR* Cmd, FOutputDevice& Ar )
{
	UE::TScopeLock ScopeLock(CriticalSection);

	FString FactorString(FParse::Token(Cmd, 0));
	float NewFactor = ( FactorString.Len() > 0 ) ? FCString::Atof(*FactorString) : GShadowmapStreamingFactor;
	if ( NewFactor >= 0.0f )
	{
		GShadowmapStreamingFactor = NewFactor;
	}
	Ar.Logf( TEXT("Shadowmap streaming factor: %.3f (lower values makes streaming more aggressive)."), GShadowmapStreamingFactor );
	return true;
}

bool FRenderAssetStreamingManager::HandleNumStreamedMipsCommand( const TCHAR* Cmd, FOutputDevice& Ar )
{
	UE::TScopeLock ScopeLock(CriticalSection);

	FString NumTextureString(FParse::Token(Cmd, 0));
	FString NumMipsString(FParse::Token(Cmd, 0));
	FString LODGroupType(FParse::Token(Cmd, false));
	int32 LODGroup = ( NumTextureString.Len() > 0 ) ? FCString::Atoi(*NumTextureString) : MAX_int32;
	int32 NumMips = ( NumMipsString.Len() > 0 ) ? FCString::Atoi(*NumMipsString) : MAX_int32;
	if ((LODGroupType == TEXT("") || LODGroupType == TEXT("Texture")) && LODGroup >= 0 && LODGroup < TEXTUREGROUP_MAX)
	{
		FTextureLODGroup& TexGroup = UDeviceProfileManager::Get().GetActiveProfile()->GetTextureLODSettings()->GetTextureLODGroup(TextureGroup(LODGroup));
		if ( NumMips >= -1 && NumMips <= MAX_TEXTURE_MIP_COUNT )
		{
			TexGroup.NumStreamedMips = NumMips;
		}
		Ar.Logf( TEXT("%s.NumStreamedMips = %d"), UTexture::GetTextureGroupString(TextureGroup(LODGroup)), TexGroup.NumStreamedMips );
	}
	else if (LODGroupType == TEXT("StaticMesh"))
	{
		// TODO
		Ar.Logf(TEXT("NumStreamedMips command is not implemented for static mesh yet"));
	}
	else if (LODGroupType == TEXT("SkeletalMesh"))
	{
		// TODO
		Ar.Logf(TEXT("NumStreamedMips command is not implemented for skeletal mesh yet"));
	}
	else
	{
		Ar.Logf( TEXT("Usage: NumStreamedMips LODGroupIndex <N> [Texture|StaticMesh|SkeletalMesh]") );
	}
	return true;
}

bool FRenderAssetStreamingManager::HandleTrackRenderAssetCommand( const TCHAR* Cmd, FOutputDevice& Ar )
{
	UE::TScopeLock ScopeLock(CriticalSection);

	FString AssetName(FParse::Token(Cmd, 0));
	if ( TrackRenderAsset(AssetName) )
	{
		Ar.Logf(TEXT("Textures or meshes containing \"%s\" are now tracked."), *AssetName);
	}
	return true;
}

bool FRenderAssetStreamingManager::HandleListTrackedRenderAssetsCommand( const TCHAR* Cmd, FOutputDevice& Ar )
{
	UE::TScopeLock ScopeLock(CriticalSection);

	FString NumAssetString(FParse::Token(Cmd, 0));
	int32 NumAssets = (NumAssetString.Len() > 0) ? FCString::Atoi(*NumAssetString) : -1;
	ListTrackedRenderAssets(Ar, NumAssets);
	return true;
}

FORCEINLINE float SqrtKeepMax(float V)
{
	return V == FLT_MAX ? FLT_MAX : FMath::Sqrt(V);
}

bool FRenderAssetStreamingManager::HandleDebugTrackedRenderAssetsCommand( const TCHAR* Cmd, FOutputDevice& Ar )
{
	UE::TScopeLock ScopeLock(CriticalSection);

	// The ENABLE_RENDER_ASSET_TRACKING macro is defined in ContentStreaming.cpp and not available here. This code does not compile any more.
#ifdef ENABLE_TEXTURE_TRACKING_BROKEN
	int32 NumTrackedTextures = GTrackedTextureNames.Num();
	if ( NumTrackedTextures )
	{
		for (int32 StreamingIndex = 0; StreamingIndex < StreamingTextures.Num(); ++StreamingIndex)
		{
			FStreamingRenderAsset& StreamingTexture = StreamingTextures[StreamingIndex];
			if (StreamingTexture.Texture)
			{
				// See if it matches any of the texture names that we're tracking.
				FString TextureNameString = StreamingTexture.Texture->GetFullName();
				const TCHAR* TextureName = *TextureNameString;
				for ( int32 TrackedTextureIndex=0; TrackedTextureIndex < NumTrackedTextures; ++TrackedTextureIndex )
				{
					const FString& TrackedTextureName = GTrackedTextureNames[TrackedTextureIndex];
					if ( FCString::Stristr(TextureName, *TrackedTextureName) != NULL )
					{
						FTrackedTextureEvent* LastEvent = NULL;
						for ( int32 LastEventIndex=0; LastEventIndex < GTrackedTextures.Num(); ++LastEventIndex )
						{
							FTrackedTextureEvent* Event = &GTrackedTextures[LastEventIndex];
							if ( FCString::Strcmp(TextureName, Event->TextureName) == 0 )
							{
								LastEvent = Event;
								break;
							}
						}

						if (LastEvent)
						{
							Ar.Logf(
								TEXT("Texture: \"%s\", ResidentMips: %d/%d, RequestedMips: %d, WantedMips: %d, DynamicWantedMips: %d, StreamingStatus: %d, StreamType: %s, Boost: %.1f"),
								TextureName,
								LastEvent->NumResidentMips,
								StreamingTexture.Texture->GetNumMips(),
								LastEvent->NumRequestedMips,
								LastEvent->WantedMips,
								LastEvent->DynamicWantedMips.ComputeMip(&StreamingTexture, MipBias, false),
								LastEvent->StreamingStatus,
								GStreamTypeNames[StreamingTexture.GetStreamingType()],
								StreamingTexture.BoostFactor
								);
						}
						else
						{
							Ar.Logf(TEXT("Texture: \"%s\", StreamType: %s, Boost: %.1f"),
								TextureName,
								GStreamTypeNames[StreamingTexture.GetStreamingType()],
								StreamingTexture.BoostFactor
								);
						}
						for( int32 HandlerIndex=0; HandlerIndex<TextureStreamingHandlers.Num(); HandlerIndex++ )
						{
							FStreamingHandlerTextureBase* TextureStreamingHandler = TextureStreamingHandlers[HandlerIndex];
							float MaxSize = 0;
							float MaxSize_VisibleOnly = 0;
							FFloatMipLevel HandlerWantedMips = TextureStreamingHandler->GetWantedSize(*this, StreamingTexture, HandlerDistance);
							Ar.Logf(
								TEXT("    Handler %s: WantedMips: %d, PerfectWantedMips: %d, Distance: %f"),
								TextureStreamingHandler->HandlerName,
								HandlerWantedMips.ComputeMip(&StreamingTexture, MipBias, false),
								HandlerWantedMips.ComputeMip(&StreamingTexture, MipBias, true),
								HandlerDistance
								);
						}

						for ( int32 LevelIndex=0; LevelIndex < ThreadSettings.LevelData.Num(); ++LevelIndex )
						{
							FRenderAssetStreamingManager::FThreadLevelData& LevelData = ThreadSettings.LevelData[ LevelIndex ].Value;
							TArray<FStreamableTextureInstance4>* TextureInstances = LevelData.ThreadTextureInstances.Find( StreamingTexture.Texture );
							if ( TextureInstances )
							{
								for ( int32 InstanceIndex=0; InstanceIndex < TextureInstances->Num(); ++InstanceIndex )
								{
									const FStreamableTextureInstance4& TextureInstance = (*TextureInstances)[InstanceIndex];
									for (int32 i = 0; i < 4; i++)
									{
										Ar.Logf(
											TEXT("    Instance: %f,%f,%f Radius: %f Range: [%f, %f] TexelFactor: %f"),
											TextureInstance.BoundsOriginX[i],
											TextureInstance.BoundsOriginY[i],
											TextureInstance.BoundsOriginZ[i],
											TextureInstance.BoundingSphereRadius[i],
											FMath::Sqrt(TextureInstance.MinDistanceSq[i]),
											SqrtKeepMax(TextureInstance.MaxDistanceSq[i]),
											TextureInstance.TexelFactor[i]
										);
									}
								}
							}
						}
					}
				}
			}
		}
	}
#endif // ENABLE_RENDER_ASSET_TRACKING

	return true;
}

bool FRenderAssetStreamingManager::HandleUntrackRenderAssetCommand( const TCHAR* Cmd, FOutputDevice& Ar )
{
	UE::TScopeLock ScopeLock(CriticalSection);

	FString AssetName(FParse::Token(Cmd, 0));
	if (UntrackRenderAsset(AssetName))
	{
		Ar.Logf(TEXT("Textures or meshes containing \"%s\" are no longer tracked."), *AssetName);
	}
	return true;
}

bool FRenderAssetStreamingManager::HandleStreamOutCommand( const TCHAR* Cmd, FOutputDevice& Ar )
{
	UE::TScopeLock ScopeLock(CriticalSection);

	FString Parameter(FParse::Token(Cmd, 0));
	int64 FreeMB = (Parameter.Len() > 0) ? FCString::Atoi(*Parameter) : 0;
	if ( FreeMB > 0 )
	{
		bool bSucceeded = StreamOutRenderAssetData( FreeMB * 1024 * 1024 );
		Ar.Logf( TEXT("Tried to stream out %llu MB of texture/mesh data: %s"), FreeMB, bSucceeded ? TEXT("Succeeded") : TEXT("Failed") );
	}
	else
	{
		Ar.Logf( TEXT("Usage: StreamOut <N> (in MB)") );
	}
	return true;
}

bool FRenderAssetStreamingManager::HandlePauseRenderAssetStreamingCommand( const TCHAR* Cmd, FOutputDevice& Ar )
{
	UE::TScopeLock ScopeLock(CriticalSection);

	bPauseRenderAssetStreaming = !bPauseRenderAssetStreaming;
	Ar.Logf( TEXT("Render asset streaming is now \"%s\"."), bPauseRenderAssetStreaming ? TEXT("PAUSED") : TEXT("UNPAUSED") );
	return true;
}

bool FRenderAssetStreamingManager::HandleStreamingManagerMemoryCommand( const TCHAR* Cmd, FOutputDevice& Ar, UWorld* InWorld )
{
	UE::TScopeLock ScopeLock(CriticalSection);
	TArray<FStreamingRenderAsset>& StreamingRenderAssets = GetStreamingRenderAssetsAsyncSafe();

	SyncStates(true);

	uint32 MemSize = sizeof(FRenderAssetStreamingManager);
	MemSize += StreamingRenderAssets.GetAllocatedSize();
	MemSize += DynamicComponentManager.GetAllocatedSize();
	MemSize += PendingStreamingRenderAssets.GetAllocatedSize()
		+ RemovedRenderAssetIndices.GetAllocatedSize();
	MemSize += LevelRenderAssetManagers.GetAllocatedSize();
	MemSize += AsyncWork->GetTask().StreamingData.GetAllocatedSize();

	for (const FLevelRenderAssetManager* LevelManager : LevelRenderAssetManagers)
	{
		if (LevelManager!=nullptr)
		{
			MemSize += LevelManager->GetAllocatedSize();
		}
	}
	
	MemSize += FSimpleStreamableAssetManager::GetAllocatedSize();

	Ar.Logf(TEXT("StreamingManagerTexture: %.2f KB used"), MemSize / 1024.0f);

	return true;
}

bool FRenderAssetStreamingManager::HandleLODGroupsCommand( const TCHAR* Cmd, FOutputDevice& Ar )
{
	UE::TScopeLock ScopeLock(CriticalSection);
	SyncStates(true);

	struct FTextureGroupStats
	{
		// Streaming texture stats
		int32 NumStreamingTextures = 0;
		uint64 CurrentTextureSize = 0;
		uint64 WantedTextureSize = 0;
		uint64 MaxTextureSize = 0;
		// Non Streaming texture stats
		int32 NumNonStreamingTextures = 0;
		uint64 NonStreamingSize = 0;
		// No resource texture
		int32 NumNoResourceTextures = 0;
	};
	FTextureGroupStats TextureGroupStats[TEXTUREGROUP_MAX];

	// Gather stats.
	for (TObjectIterator<UTexture> It; It; ++It)
	{
		UTexture* Texture = *It;
		check(Texture);

		FTextureGroupStats& LODStats = TextureGroupStats[Texture->LODGroup];

		const EPixelFormat PixelFormat = [&]()->EPixelFormat
		{
			if (Texture->GetRunningPlatformData() && *Texture->GetRunningPlatformData())
			{
				return (*Texture->GetRunningPlatformData())->PixelFormat;
			}
			else if (Texture->GetResource() && Texture->GetResource()->TextureRHI)
			{
				return Texture->GetResource()->TextureRHI->GetFormat();
			}
			else
			{
				return PF_Unknown;
			}
		}();

		// No resource no size taken
		if (!Texture->GetResource())
		{
			LODStats.NumNoResourceTextures++;
		}
		else if (Texture->IsStreamable())
		{
			FStreamingRenderAsset* StreamingTexture = GetStreamingRenderAsset(Texture);
			if (ensure(StreamingTexture))
			{
				LODStats.NumStreamingTextures++;
				LODStats.CurrentTextureSize += StreamingTexture->GetSize(StreamingTexture->ResidentMips);;
				LODStats.WantedTextureSize += StreamingTexture->GetSize(StreamingTexture->WantedMips);
				LODStats.MaxTextureSize += StreamingTexture->GetSize(StreamingTexture->MaxAllowedMips);
			}
		}
		else
		{
			LODStats.NumNonStreamingTextures++;
			LODStats.NonStreamingSize += Texture->CalcTextureMemorySizeEnum(TMC_ResidentMips);
		}
	}

	// Output stats.
	{
		UE_LOGF(LogContentStreaming, Log, "Texture memory usage:");
		FTextureGroupStats TotalStats;
		for (int32 GroupIndex = 0; GroupIndex < TEXTUREGROUP_MAX; ++GroupIndex)
		{
			FTextureGroupStats& Stat = TextureGroupStats[GroupIndex];
			if (Stat.NumStreamingTextures || Stat.NumNonStreamingTextures || Stat.NumNoResourceTextures)
			{
				TotalStats.NumStreamingTextures += Stat.NumStreamingTextures;
				TotalStats.NumNonStreamingTextures += Stat.NumNonStreamingTextures;
				TotalStats.CurrentTextureSize += Stat.CurrentTextureSize;
				TotalStats.WantedTextureSize += Stat.WantedTextureSize;
				TotalStats.MaxTextureSize += Stat.MaxTextureSize;
				TotalStats.NonStreamingSize += Stat.NonStreamingSize;
				TotalStats.NumNoResourceTextures += Stat.NumNoResourceTextures;
				UE_LOGF(LogContentStreaming, Log, "%34ls: NumStreamingTextures=%4d { Current=%8.1f KB, Wanted=%8.1f KB, OnDisk=%8.1f KB }, NumNonStreaming=%4d { Size=%8.1f KB }, NumWithNoResource=%4d",
					UTexture::GetTextureGroupString((TextureGroup)GroupIndex),
					Stat.NumStreamingTextures,
					Stat.CurrentTextureSize / 1024.0f,
					Stat.WantedTextureSize / 1024.0f,
					Stat.MaxTextureSize / 1024.0f,
					Stat.NumNonStreamingTextures,
					Stat.NonStreamingSize / 1024.0f,
					Stat.NumNoResourceTextures);
			}
		}
		UE_LOGF(LogContentStreaming, Log, "%34ls: NumStreamingTextures=%4d { Current=%8.1f KB, Wanted=%8.1f KB, OnDisk=%8.1f KB }, NumNonStreaming=%4d { Size=%8.1f KB }, NumWithNoResource=%4d",
			TEXT("Total"),
			TotalStats.NumStreamingTextures,
			TotalStats.CurrentTextureSize / 1024.0f,
			TotalStats.WantedTextureSize / 1024.0f,
			TotalStats.MaxTextureSize / 1024.0f,
			TotalStats.NumNonStreamingTextures,
			TotalStats.NonStreamingSize / 1024.0f,
			TotalStats.NumNoResourceTextures);
	}
	return true;
}

bool FRenderAssetStreamingManager::HandleInvestigateRenderAssetCommand(const TCHAR* Cmd, FOutputDevice& Ar, UWorld* InWorld)
{
	UE::TScopeLock ScopeLock(CriticalSection);
	TArray<FStreamingRenderAsset>& StreamingRenderAssets = GetStreamingRenderAssetsAsyncSafe();
	SyncStates(true);

	FString InvestigateAssetName(FParse::Token(Cmd, 0));
	if (InvestigateAssetName.Len())
	{
		FAsyncRenderAssetStreamingData& StreamingData = AsyncWork->GetTask().StreamingData;
		
		FSimpleStreamableAssetManager::FScopedLock ScopedLock(FSimpleStreamableAssetManager::GetCriticalSection(), FSimpleStreamableAssetManager::IsEnabled());
		FSimpleStreamableAssetManager::Process();

		if (FSimpleStreamableAssetManager::IsEnabled())
		{
			StreamingData.Init(CurrentViewInfos, LastWorldUpdateTime);
		}
		else
		{
			StreamingData.Init(CurrentViewInfos, LastWorldUpdateTime, LevelRenderAssetManagers, DynamicComponentManager);
		}

		StreamingData.ComputeViewInfoExtras(Settings);
		StreamingData.UpdateBoundSizes_Async(Settings);

		const bool bInvestigateAll = InvestigateAssetName == TEXT("*");
		TArray<int32> AssetIndices;
		AssetIndices.Reserve(StreamingRenderAssets.Num());
		for (int32 AssetIndex = 0; AssetIndex < StreamingRenderAssets.Num(); ++AssetIndex)
		{
			AssetIndices.Add(AssetIndex);
		}

		// When investigating all assets, sort render asset based on their full name to facilitate diffing
		if (bInvestigateAll)
		{
			AssetIndices.Sort([&StreamingRenderAssets](const int32& A, const int32& B)
			{
				return StreamingRenderAssets[A].RenderAsset->GetFullName() < StreamingRenderAssets[B].RenderAsset->GetFullName();
			});
		}

		for (int32 i = 0; i < AssetIndices.Num(); ++i)
		{
			int32 AssetIndex = AssetIndices[i];
			FStreamingRenderAsset& StreamingRenderAsset = StreamingRenderAssets[AssetIndex];
			FString AssetName = StreamingRenderAsset.RenderAsset->GetFullName();
			if (bInvestigateAll || AssetName.Contains(InvestigateAssetName))
			{
				UStreamableRenderAsset* RenderAsset = StreamingRenderAsset.RenderAsset;
				if (!RenderAsset) continue;
				const EStreamableRenderAssetType AssetType = StreamingRenderAsset.RenderAssetType;
				const FStreamableRenderResourceState ResourceState = RenderAsset->GetStreamableResourceState();
				UTexture* Texture = Cast<UTexture>(RenderAsset);
				UStaticMesh* StaticMesh = Cast<UStaticMesh>(RenderAsset);
				int32 CurrentMipIndex = ResourceState.LODCountToAssetFirstLODIdx(StreamingRenderAsset.ResidentMips);
				int32 WantedMipIndex = ResourceState.LODCountToAssetFirstLODIdx(StreamingRenderAsset.GetPerfectWantedMips());
				int32 MaxMipIndex = ResourceState.LODCountToAssetFirstLODIdx(StreamingRenderAsset.MaxAllowedMips);

				UE_LOGF(LogContentStreaming, Log, "%ls: %ls", FStreamingRenderAsset::GetStreamingAssetTypeStr(AssetType), *AssetName);
				FString LODGroupName = Texture ? UTexture::GetTextureGroupString((TextureGroup)StreamingRenderAsset.LODGroup) : TEXT("Unknown");
#if WITH_EDITORONLY_DATA
				if (StaticMesh)
				{
					LODGroupName = StaticMesh->GetLODGroup().ToString();
				}
#endif
				// put this up in some shared code somewhere in FGenericPlatformMemory
				const TCHAR* BucketNames[] = { TEXT("Largest"), TEXT("Larger"), TEXT("Default"), TEXT("Smaller"), TEXT("Smallest"), TEXT("Tiniest") };
				if ((int32)FPlatformMemory::GetMemorySizeBucket() < UE_ARRAY_COUNT(BucketNames))
				{
					UE_LOGF(LogContentStreaming, Log, "  LOD group:       %ls [Bucket=%ls]", *LODGroupName, BucketNames[(int32)FPlatformMemory::GetMemorySizeBucket()]);
				}
				else
				{
					UE_LOGF(LogContentStreaming, Log, "  LOD group:       %ls [Unkown Bucket]", *LODGroupName);
				}
				if (Texture && Texture->GetRunningPlatformData() && *Texture->GetRunningPlatformData())
				{
					UE_LOGF(LogContentStreaming, Log, "  Format:          %ls", GPixelFormats[(*Texture->GetRunningPlatformData())->PixelFormat].Name);
				}
				if (RenderAsset->bGlobalForceMipLevelsToBeResident)
				{
					UE_LOGF(LogContentStreaming, Log, "  Force all mips:  bGlobalForceMipLevelsToBeResident");
				}
				else if (RenderAsset->bForceMiplevelsToBeResident)
				{
					UE_LOGF(LogContentStreaming, Log, "  Force all mips:  bForceMiplevelsToBeResident");
				}
				else if (RenderAsset->ShouldMipLevelsBeForcedResident())
				{
					float TimeLeft = (float)(RenderAsset->ForceMipLevelsToBeResidentTimestamp - FApp::GetCurrentTime());
					UE_LOGF(LogContentStreaming, Log, "  Force all mips:  %.1f seconds left", FMath::Max(TimeLeft, 0.0f));
				}
				else if (StreamingRenderAsset.GetFlag(FStreamingRenderAssetFlags::ForceFullyLoadHeuristic))
				{
					UE_LOGF(LogContentStreaming, Log, "  Force all mips:  bForceFullyLoad");
				}
				else if (ResourceState.MaxNumLODs == 1)
				{
					UE_LOGF(LogContentStreaming, Log, "  Force all mips:  No mip-maps");
				}
				
				if (Texture && !Texture->IsCurrentlyVirtualTextured() && Texture->GetRunningPlatformData() && *Texture->GetRunningPlatformData())
				{
					const TIndirectArray<struct FTexture2DMipMap>& TextureMips = (*Texture->GetRunningPlatformData())->Mips;
					UTexture2D* Texture2D = Cast<UTexture2D>(Texture);
					UVolumeTexture* VolumeTexture = Cast<UVolumeTexture>(Texture);
					UTexture2DArray* Texture2DArray = Cast<UTexture2DArray>(Texture);
					if (Texture2D)
					{
						UE_LOGF(LogContentStreaming, Log, "  Current size [2D Mips]: %dx%d [%d]", TextureMips[CurrentMipIndex].SizeX, TextureMips[CurrentMipIndex].SizeY, StreamingRenderAsset.ResidentMips);
						UE_LOGF(LogContentStreaming, Log, "  Wanted size [2D Mips]:  %dx%d [%d]", TextureMips[WantedMipIndex].SizeX, TextureMips[WantedMipIndex].SizeY, StreamingRenderAsset.GetPerfectWantedMips());
					}
					else if (VolumeTexture)
					{
						UE_LOGF(LogContentStreaming, Log, "  Current size [3D Mips]: %dx%dx%d [%d]", TextureMips[CurrentMipIndex].SizeX, TextureMips[CurrentMipIndex].SizeY, TextureMips[CurrentMipIndex].SizeZ, StreamingRenderAsset.ResidentMips);
						UE_LOGF(LogContentStreaming, Log, "  Wanted size [3D Mips]:  %dx%dx%d [%d]", TextureMips[WantedMipIndex].SizeX, TextureMips[WantedMipIndex].SizeY, TextureMips[CurrentMipIndex].SizeZ, StreamingRenderAsset.GetPerfectWantedMips());
					}
					else if (Texture2DArray)
					{
						UE_LOGF(LogContentStreaming, Log, "  Current size [2D Array Mips]: %dx%d*%d [%d]", TextureMips[CurrentMipIndex].SizeX, TextureMips[CurrentMipIndex].SizeY, TextureMips[CurrentMipIndex].SizeZ, StreamingRenderAsset.ResidentMips);
						UE_LOGF(LogContentStreaming, Log, "  Wanted size [2D Array Mips]:  %dx%d*%d [%d]", TextureMips[WantedMipIndex].SizeX, TextureMips[WantedMipIndex].SizeY, TextureMips[CurrentMipIndex].SizeZ, StreamingRenderAsset.GetPerfectWantedMips());
					}
				}
				else
				{
					UE_LOGF(LogContentStreaming, Log, "  Current LOD index: %d", CurrentMipIndex);
					UE_LOGF(LogContentStreaming, Log, "  Wanted LOD index: %d", WantedMipIndex);
				}
				UE_LOGF(LogContentStreaming, Log, "  Allowed mips:        %d-%d [%d]", StreamingRenderAsset.MinAllowedMips, StreamingRenderAsset.MaxAllowedMips, ResourceState.MaxNumLODs);
				UE_LOGF(LogContentStreaming, Log, "  LoadOrder Priority:  %d", StreamingRenderAsset.LoadOrderPriority);
				UE_LOGF(LogContentStreaming, Log, "  Retention Priority:  %d", StreamingRenderAsset.RetentionPriority);
				UE_LOGF(LogContentStreaming, Log, "  Boost factor:        %.1f", StreamingRenderAsset.BoostFactor);

				FString BiasDesc;
				{
					int32 CumuBias = 0;
					if (!Settings.bUseAllMips)
					{
						// UI specific Bias : see UTextureLODSettings::CalculateLODBias(), included in CachedCombinedLODBias.
						extern int32 GUITextureLODBias;
						if (StreamingRenderAsset.LODGroup == TEXTUREGROUP_UI && GUITextureLODBias)
						{
							BiasDesc += FString::Printf(TEXT(" [UI:%d]"), GUITextureLODBias);
							CumuBias += GUITextureLODBias;
						}

						// LOD group Bias : see UTextureLODSettings::CalculateLODBias(), included in CachedCombinedLODBias
						const FTextureLODGroup& LODGroupInfo = UDeviceProfileManager::Get().GetActiveProfile()->GetTextureLODSettings()->TextureLODGroups[StreamingRenderAsset.LODGroup];
						const bool bIsVirtualTexture = Texture && Texture->IsCurrentlyVirtualTextured();
						const int32 GroupLODBias = bIsVirtualTexture ? LODGroupInfo.LODBias_VT : LODGroupInfo.LODBias;
						if (GroupLODBias)
						{
							if (FPlatformProperties::RequiresCookedData())
							{
								BiasDesc += FString::Printf(TEXT(" [LODGroup.Bias:0(%d)]"), GroupLODBias);
							}
							else
							{
								BiasDesc += FString::Printf(TEXT(" [LODGroup.Bias:%d]"), GroupLODBias);
								CumuBias += GroupLODBias;
							}
						}

						// LOD group MaxResolution clamp : see UTextureLODSettings::CalculateLODBias(), included in CachedCombinedLODBias
						const int32 MipCountBeforeMaxRes = ResourceState.MaxNumLODs - RenderAsset->NumCinematicMipLevels -
							(StreamingRenderAsset.LODGroup == TEXTUREGROUP_UI ? GUITextureLODBias : 0) - 
							(FPlatformProperties::RequiresCookedData() ? 0 : (GroupLODBias + (Texture ? Texture->LODBias : 0)));
						const int32 MaxResBias = MipCountBeforeMaxRes - (LODGroupInfo.MaxLODMipCount + 1);
						if (MaxResBias > 0)
						{
							BiasDesc += FString::Printf(TEXT(" [LODGroup.MaxRes:%d]"), MaxResBias);
							CumuBias += MaxResBias;
						}

						// Asset LODBias : see UTextureLODSettings::CalculateLODBias(), included in CachedCombinedLODBias
						if (Texture && Texture->LODBias)
						{
							if (FPlatformProperties::RequiresCookedData())
							{
								BiasDesc += FString::Printf(TEXT(" [Asset.Bias:0(%d)]"), Texture->LODBias);
							}
							else
							{
								BiasDesc += FString::Printf(TEXT(" [Asset.Bias:%d]"), Texture->LODBias);
								CumuBias += Texture->LODBias;
							}
						}

						// Asset cinematic mips : see UTextureLODSettings::CalculateLODBias() and FStreamingRenderAsset::UpdateDynamicData(), included in CachedCombinedLODBias
						if (RenderAsset->NumCinematicMipLevels && !(StreamingRenderAsset.GetFlag(FStreamingRenderAssetFlags::ForceFullyLoad) && RenderAsset->bUseCinematicMipLevels))
						{
							BiasDesc += FString::Printf(TEXT(" [Asset.Cine:%d]"), RenderAsset->NumCinematicMipLevels);
							CumuBias += RenderAsset->NumCinematicMipLevels;
						}
					}

					// RHI max resolution and optional mips : see FStreamingRenderAsset::UpdateDynamicData().
					{
						const int32 OptMipBias = ResourceState.MaxNumLODs - CumuBias - RenderAsset->GetStreamableResourceState().NumNonOptionalLODs;
						if (StreamingRenderAsset.OptionalMipsState != FStreamingRenderAsset::EOptionalMipsState::OMS_HasOptionalMips && OptMipBias > 0)
						{
							BiasDesc += FString::Printf(TEXT(" [Asset.Opt:%d]"), OptMipBias);
							CumuBias += OptMipBias;
						}

						const int32 RHIMaxResBias = ResourceState.MaxNumLODs - CumuBias - GMaxTextureMipCount;
						if (RHIMaxResBias > 0)
						{
							BiasDesc += FString::Printf(TEXT(" [RHI.MaxRes:%d]"), RHIMaxResBias);
							CumuBias += RHIMaxResBias;
						}
					}

					// Memory budget bias
					// Global Bias : see FStreamingRenderAsset::UpdateDynamicData().
					if (StreamingRenderAsset.IsMaxResolutionAffectedByGlobalBias() && !Settings.bUsePerTextureBias && Settings.GlobalMipBias)
					{
						BiasDesc += FString::Printf(TEXT(" [Global:%d]"), Settings.GlobalMipBias);
						CumuBias += Settings.GlobalMipBias;
					}
					
					if (StreamingRenderAsset.BudgetMipBias)
					{
						BiasDesc += FString::Printf(TEXT(" [Budget:%d]"), StreamingRenderAsset.BudgetMipBias);
						CumuBias += StreamingRenderAsset.BudgetMipBias;
					}
				}

				UE_LOGF(LogContentStreaming, Log, "  Mip bias:			  %d %ls", ResourceState.MaxNumLODs - StreamingRenderAsset.MaxAllowedMips, !BiasDesc.IsEmpty() ? *BiasDesc : TEXT("") );

				if (InWorld && !GIsEditor)
				{
					UE_LOGF(LogContentStreaming, Log, "  Time: World=%.3f LastUpdate=%.3f ", InWorld->GetTimeSeconds(), LastWorldUpdateTime);
				}

				for (int32 ViewIndex = 0; ViewIndex < StreamingData.GetViewInfos().Num(); ViewIndex++)
				{
					// Calculate distance of viewer to bounding sphere.
					const FStreamingViewInfo& ViewInfo = StreamingData.GetViewInfos()[ViewIndex];
					UE_LOGF(LogContentStreaming, Log, "  View%d: Position=(%ls) ScreenSize=%f MaxEffectiveScreenSize=%f Boost=%f", ViewIndex, *ViewInfo.ViewOrigin.ToString(), ViewInfo.ScreenSize, Settings.MaxEffectiveScreenSize, ViewInfo.BoostFactor);
				}

				StreamingData.UpdatePerfectWantedMips_Async(StreamingRenderAsset, Settings, true);
			}
		}
	}
	else
	{
		Ar.Logf(TEXT("Usage: InvestigateTexture <name>"));
	}
	return true;
}
#endif // !UE_BUILD_SHIPPING
/**
 * Allows the streaming manager to process exec commands.
 * @param InWorld	world contexxt
 * @param Cmd		Exec command
 * @param Ar		Output device for feedback
 * @return			true if the command was handled
 */
bool FRenderAssetStreamingManager::Exec( UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar )
{
#if !UE_BUILD_SHIPPING
	if (FParse::Command(&Cmd,TEXT("DumpTextureStreamingStats"))
		|| FParse::Command(&Cmd, TEXT("DumpRenderAssetStreamingStats")))
	{
		return HandleDumpTextureStreamingStatsCommand( Cmd, Ar );
	}
	if (FParse::Command(&Cmd,TEXT("ListStreamingTextures"))
		|| FParse::Command(&Cmd, TEXT("ListStreamingRenderAssets")))
	{
		return HandleListStreamingRenderAssetsCommand( Cmd, Ar );
	}
	if (FParse::Command(&Cmd, TEXT("ResetMaxEverRequiredTextures"))
		|| FParse::Command(&Cmd, TEXT("ResetMaxEverRequiredRenderAssetMemory")))
	{
		return HandleResetMaxEverRequiredRenderAssetMemoryCommand(Cmd, Ar);
	}
	if (FParse::Command(&Cmd,TEXT("LightmapStreamingFactor")))
	{
		return HandleLightmapStreamingFactorCommand( Cmd, Ar );
	}
	else if (FParse::Command(&Cmd,TEXT("CancelTextureStreaming"))
		|| FParse::Command(&Cmd, TEXT("CancelRenderAssetStreaming")))
	{
		return HandleCancelRenderAssetStreamingCommand( Cmd, Ar );
	}
	else if (FParse::Command(&Cmd,TEXT("ShadowmapStreamingFactor")))
	{
		return HandleShadowmapStreamingFactorCommand( Cmd, Ar );
	}
	else if (FParse::Command(&Cmd,TEXT("NumStreamedMips")))
	{
		return HandleNumStreamedMipsCommand( Cmd, Ar );
	}
	else if (FParse::Command(&Cmd,TEXT("TrackTexture"))
		|| FParse::Command(&Cmd, TEXT("TrackRenderAsset")))
	{
		return HandleTrackRenderAssetCommand( Cmd, Ar );
	}
	else if (FParse::Command(&Cmd,TEXT("ListTrackedTextures"))
		|| FParse::Command(&Cmd, TEXT("ListTrackedRenderAssets")))
	{
		return HandleListTrackedRenderAssetsCommand( Cmd, Ar );
	}
	else if (FParse::Command(&Cmd,TEXT("DebugTrackedTextures"))
		|| FParse::Command(&Cmd, TEXT("DebugTrackedRenderAssets")))
	{
		return HandleDebugTrackedRenderAssetsCommand( Cmd, Ar );
	}
	else if (FParse::Command(&Cmd,TEXT("UntrackTexture"))
		|| FParse::Command(&Cmd, TEXT("UntrackRenderAsset")))
	{
		return HandleUntrackRenderAssetCommand( Cmd, Ar );
	}
	else if (FParse::Command(&Cmd,TEXT("StreamOut")))
	{
		return HandleStreamOutCommand( Cmd, Ar );
	}
	else if (FParse::Command(&Cmd,TEXT("PauseTextureStreaming"))
		|| FParse::Command(&Cmd, TEXT("PauseRenderAssetStreaming")))
	{
		return HandlePauseRenderAssetStreamingCommand( Cmd, Ar );
	}
	else if (FParse::Command(&Cmd,TEXT("StreamingManagerMemory")))
	{
		return HandleStreamingManagerMemoryCommand( Cmd, Ar, InWorld );
	}
	else if (FParse::Command(&Cmd,TEXT("TextureGroups"))
		|| FParse::Command(&Cmd, TEXT("LODGroups")))
	{
		return HandleLODGroupsCommand( Cmd, Ar );
	}
	else if (FParse::Command(&Cmd,TEXT("InvestigateTexture"))
		|| FParse::Command(&Cmd, TEXT("InvestigateRenderAsset")))
	{
		return HandleInvestigateRenderAssetCommand( Cmd, Ar, InWorld );
	}
	else if (FParse::Command(&Cmd,TEXT("ListMaterialsWithMissingTextureStreamingData")))
	{
		Ar.Logf(TEXT("Listing all materials with not texture streaming data."));
		Ar.Logf(TEXT("Run \"BuildMaterialTextureStreamingData\" in the editor to fix the issue"));
		Ar.Logf(TEXT("Note that some materials might have no that even after rebuild."));
		for( TObjectIterator<UMaterialInterface> It; It; ++It )
		{
			UMaterialInterface* Material = *It;
			if (Material && Material->GetOutermost() != GetTransientPackage() && Material->HasAnyFlags(RF_Public) && Material->UseAnyStreamingTexture() && !Material->HasTextureStreamingData()) 
			{
				FString TextureName = Material->GetFullName();
				Ar.Logf(TEXT("%s"), *TextureName);
			}
		}
		return true;
	}
#endif // !UE_BUILD_SHIPPING

	return false;
}

float FRenderAssetStreamingManagerHelper::GetWorldTime(UWorld* InWorld)
{
	// When paused, updating the world time sometimes break visibility logic.
	if (InWorld && !InWorld->IsPaused())
	{
		// In the editor, we only return a time for the PIE world.
		if (!GIsEditor || (InWorld->IsPlayInEditor() && InWorld->Scene && InWorld->Scene->GetFrameNumber()))
		{
			return InWorld->GetTimeSeconds();
		}
	}

	return 0.f;
}
