// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
TextureStreamingManager.h: Definitions of classes used for texture streaming.
=============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "ContentStreaming.h"
#include "AbandonedRenderAssetUpdateManager.h"
#include "Streaming/TextureStreamingHelpers.h"
#include "Streaming/StreamingTexture.h"
#include "Streaming/LevelTextureManager.h"
#include "Streaming/TextureInstanceTask.h"

class AActor;
class FRenderAssetStreamingMipCalcTask;
class IPakFile;
class UPrimitiveComponent;

template<typename TTask> class FAsyncTask;

/*-----------------------------------------------------------------------------
	Texture or mesh streaming.
-----------------------------------------------------------------------------*/

/**
 * Streaming manager dealing with textures/meshes.
 */
struct FRenderAssetStreamingManager final : public IRenderAssetStreamingManager
{
	/** Constructor, initializing all members */
	FRenderAssetStreamingManager();

	virtual ~FRenderAssetStreamingManager();

	/** Called before GC to clear pending kill levels. */
	void OnPreGarbageCollect();

	/**
	 * Updates streaming, taking into account all current view infos. Can be called multiple times per frame.
	 *
	 * @param DeltaTime				Time since last call in seconds
	 * @param bProcessEverything	[opt] If true, process all resources with no throttling limits
	 */
	virtual void UpdateResourceStreaming( float DeltaTime, bool bProcessEverything=false ) override;

	/**
	 * Updates streaming for an individual texture/mesh, taking into account all view infos.
	 *
	 * @param RenderAsset	Texture or mesh to update
	 */
	virtual void UpdateIndividualRenderAsset( UStreamableRenderAsset* RenderAsset ) override;

	/**
	 * Register an asset whose non-resident mips need to be loaded ASAP when visible.
	 * bIgnoreStreamingMipBias must be set on the asset.
	 * Either bForceMiplevelsToBeResident or ForceMipLevelsToBeResidentTimestamp need to be set on the asset.
	 *
	 * @param RenderAsset The asset to register
	 * @return bool True if the streaming request is successful
	 */
	virtual bool FastForceFullyResident(UStreamableRenderAsset* RenderAsset) override;

	/**
	 * Blocks till all pending requests are fulfilled.
	 *
	 * @param TimeLimit		Optional time limit for processing, in seconds. Specifying 0 means infinite time limit.
	 * @param bLogResults	Whether to dump the results to the log.
	 * @return				Number of streaming requests still in flight, if the time limit was reached before they were finished.
	 */
	virtual int32 BlockTillAllRequestsFinished( float TimeLimit = 0.0f, bool bLogResults = false ) override;

	/**
	 * Cancels the timed Forced resources (i.e used the Kismet action "Stream In Textures").
	 */
	virtual void CancelForcedResources() override;

	/**
	 * Notifies manager of "level" change so it can prioritize character textures for a few frames.
	 */
	virtual void NotifyLevelChange() override;

	/** Don't stream world resources for the next NumFrames. */
	virtual void SetDisregardWorldResourcesForFrames( int32 NumFrames ) override;

	/**
	 *	Try to stream out texture/mesh mip-levels to free up more memory.
	 *	@param RequiredMemorySize	- Required minimum available texture memory
	 *	@return						- Whether it succeeded or not
	 **/
	virtual bool StreamOutRenderAssetData( int64 RequiredMemorySize ) override;

	virtual int64 GetMemoryOverBudget() const override { return MemoryOverBudget; }

	/** Pool size for streaming. */
	virtual int64 GetPoolSize() const override;

	virtual int64 GetRequiredPoolSize() const override { return DisplayedStats.RequiredPool; }

	virtual int64 GetMaxEverRequired() const override { return MaxEverRequired; }

	virtual float GetCachedMips() const override { return DisplayedStats.CachedMips; }

	virtual void ResetMaxEverRequired() override { MaxEverRequired = 0; }

	/**
	 * Allows the streaming manager to process exec commands.
	 *
	 * @param InWorld World context
	 * @param Cmd	Exec command
	 * @param Ar	Output device for feedback
	 * @return		true if the command was handled
	 */
	virtual bool Exec( UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar ) override;

	/**
	 * Exec command handlers
	 */
#if !UE_BUILD_SHIPPING
	bool HandleDumpTextureStreamingStatsCommand(const TCHAR* Cmd, FOutputDevice& Ar);
	bool HandleListStreamingRenderAssetsCommand(const TCHAR* Cmd, FOutputDevice& Ar);
	bool HandleResetMaxEverRequiredRenderAssetMemoryCommand(const TCHAR* Cmd, FOutputDevice& Ar);
	bool HandleLightmapStreamingFactorCommand( const TCHAR* Cmd, FOutputDevice& Ar );
	bool HandleCancelRenderAssetStreamingCommand( const TCHAR* Cmd, FOutputDevice& Ar );
	bool HandleShadowmapStreamingFactorCommand( const TCHAR* Cmd, FOutputDevice& Ar );
	bool HandleNumStreamedMipsCommand( const TCHAR* Cmd, FOutputDevice& Ar );
	bool HandleTrackRenderAssetCommand( const TCHAR* Cmd, FOutputDevice& Ar );
	bool HandleListTrackedRenderAssetsCommand( const TCHAR* Cmd, FOutputDevice& Ar );
	bool HandleDebugTrackedRenderAssetsCommand( const TCHAR* Cmd, FOutputDevice& Ar );
	bool HandleUntrackRenderAssetCommand( const TCHAR* Cmd, FOutputDevice& Ar );
	bool HandleStreamOutCommand( const TCHAR* Cmd, FOutputDevice& Ar );
	bool HandlePauseRenderAssetStreamingCommand( const TCHAR* Cmd, FOutputDevice& Ar );
	bool HandleStreamingManagerMemoryCommand( const TCHAR* Cmd, FOutputDevice& Ar, UWorld* InWorld );
	bool HandleLODGroupsCommand( const TCHAR* Cmd, FOutputDevice& Ar );
	bool HandleInvestigateRenderAssetCommand( const TCHAR* Cmd, FOutputDevice& Ar, UWorld* InWorld );
#endif // !UE_BUILD_SHIPPING

	/** Adds a new texture/mesh to the streaming manager. */
	virtual void AddStreamingRenderAsset( UStreamableRenderAsset* RenderAsset ) override;

	/** Removes a texture/mesh from the streaming manager. */
	virtual void RemoveStreamingRenderAsset( UStreamableRenderAsset* RenderAsset ) override;

	/** Only call on the game thread. */
	virtual bool IsFullyStreamedIn(UStreamableRenderAsset* RenderAsset) override;

	virtual bool TryRegisterStreamOutProtectedAsset(UStreamableRenderAsset* Asset) override;
	virtual void UnregisterStreamOutProtectedAsset(UStreamableRenderAsset* Asset) override;

	/** Calls Asset->StreamOut() unless the asset is registered as protected.
	 *  The read lock is held across the StreamOut call.
	 *  Returns Asset->StreamOut()'s result, or false if the asset is protected. */
	bool StreamOutProtected(UStreamableRenderAsset* Asset, int32 NewMipCount);

	/** Adds a ULevel to the streaming manager. */
	virtual void AddLevel( class ULevel* Level ) override;

	/** Removes a ULevel from the streaming manager. */
	virtual void RemoveLevel( class ULevel* Level ) override;

	/* Notifies manager that level primitives were shifted */
	virtual void NotifyLevelOffset(ULevel* Level, const FVector& Offset) override;
	/** Called when a spawned actor is destroyed. */
	virtual void NotifyActorDestroyed( AActor* Actor ) override;

	/** Called when a primitive is detached from an actor or another component. */
	virtual void NotifyPrimitiveDetached( const IPrimitiveComponent* Primitive ) override;

	/** Called when a primitive streaming data needs to be updated. */
	virtual void NotifyPrimitiveUpdated( const IPrimitiveComponent* Primitive ) override;

	/**  Called when a primitive streaming data needs to be updated in the last stage of the frame. */
	virtual void NotifyPrimitiveUpdated_Concurrent( const IPrimitiveComponent* Primitive ) override;

	/** Returns the corresponding FStreamingRenderAsset for a texture or mesh. */
	FStreamingRenderAsset* GetStreamingRenderAsset( const UStreamableRenderAsset* RenderAsset );

	/** Set current pause state for texture/mesh streaming */
	virtual void PauseRenderAssetStreaming(bool bInShouldPause) override
	{
		bPauseRenderAssetStreaming = bInShouldPause;
	}

	/** Return all bounds related to the ref object */
	virtual void GetObjectReferenceBounds(const UObject* RefObject, TArray<FBox>& AssetBoxes) override;

	virtual void GetAssetComponents(const UStreamableRenderAsset* RenderAsset, TArray<const IPrimitiveComponent*>& OutComps, TFunction<bool(const IPrimitiveComponent*)> ShouldChoose) override;

	/** Propagates a change to the active lighting scenario. */
	void PropagateLightingScenarioChange() override;

	void AddRenderedTextureStats(TMap<FString, FRenderedTextureStats>& InOutRenderedTextureStats) override;

	void OnRenderAssetUpdateAbandoned(UStreamableRenderAsset* OwningRenderAsset, TRefCountPtr<class FRenderAssetUpdate>& RenderAssetUpdate) override;

	/**
	 * Mark the textures/meshes with a timestamp. They're about to lose their location-based heuristic and we don't want them to
	 * start using LastRenderTime heuristic for a few seconds until they are garbage collected!
	 *
	 * @param RemovedRenderAssets	List of removed textures or meshes.
	 */
	void SetRenderAssetsRemovedTimestamp(const FRemovedRenderAssetArray& RemovedRenderAssets);

	/* Schedules file recache */
	virtual void InvalidateFileCache() override;

	/** Whether it is allowed for this manager to use more cores using ParallelFor when doing the incremental update. */
	static bool IsParallelForAllowedDuringIncrementalUpdate(int32 NumElementsToProcess, int32& OutParallelForMinBatchSize);

private:
//BEGIN: Thread-safe functions and data
		friend class FRenderAssetStreamingMipCalcTask;
		friend class FUpdateStreamingRenderAssetsTask;

		/** Remove any references in level managers to this component */
		void RemoveStaticReferences(const IPrimitiveComponent* Primitive);

		/**
		 * Not thread-safe: Updates a portion (as indicated by 'StageIndex') of all streaming textures/meshes,
		 * allowing their streaming state to progress.
		 *
		 * @param StageIndex		Current stage index
		 * @param NumUpdateStages	Number of texture/mesh update stages
		 * @Param bAsync			Whether this is called on an async task
		 */
		void UpdateStreamingRenderAssets(int32 StageIndex, int32 NumStages, bool bAsync = false);

		/** Check visibility of fast response assets and initiate stream-in requests if necessary. */
		void TickFastResponseAssets();

		void ProcessRemovedRenderAssets();
		void ProcessAddedRenderAssets();
		void ConditionalUpdateStaticData();
		void ProcessLevelsToReferenceToStreamedTextures();

		/** Adds new textures/meshes and level data on the gamethread (while the worker thread isn't active). */
		void PrepareAsyncTask( bool bProcessEverything );

		/** Checks for updates in the user settings (CVars, etc). */
		void CheckUserSettings();

		/**
		 * Temporarily boosts the streaming distance factor by the specified number.
		 * This factor is automatically reset to 1.0 after it's been used for mip-calculations.
		 */
		void BoostTextures( AActor* Actor, float BoostFactor ) override;

		/**
		 * Updates the state of a texture with information about it's optional bulkdata
		 */
		void SetOptionalBulkData( UTexture* Texture, bool bHasOptionalBulkData );

		/**
		 * Stream textures/meshes in/out, based on the priorities calculated by the async work.
		 *
		 * @param bProcessEverything	Whether we're processing all textures in one go
		 */
		void StreamRenderAssets( bool bProcessEverything );

		TConstArrayView<int32> GetNumStreamedMipsArray(EStreamableRenderAssetType AssetType) const
		{
			switch (AssetType)
			{
			case EStreamableRenderAssetType::Texture:
				return MakeArrayView(NumStreamedMips_Texture);
			case EStreamableRenderAssetType::StaticMesh:
				return MakeArrayView(NumStreamedMips_StaticMesh);
			case EStreamableRenderAssetType::SkeletalMesh:
				return MakeArrayView(NumStreamedMips_SkeletalMesh);
			default:
				check(false);
				return TConstArrayView<int32>();
			}
		}

		/** All streaming texture or mesh objects. 
		* Use directly only if it's safe to overlap with UpdateStreamingRenderAssets.
		* For an async safe version, use GetStreamingRenderAssetsAsyncSafe
		*/
		TArray<FStreamingRenderAsset> AsyncUnsafeStreamingRenderAssets;

		/** 
		 * KeyFunc to ensure lookups in the TSet below can be done with raw pointers.
		 * Under certain circumstances, lookups can be performed on ReferencedRenderAssets with invalid pointers (freed resources).
		 * Constructing a TObjectPtr<UStreamableRenderAsset>() from an invalid pointer is unsupported.
		 */
		struct FPointerKeyFunc : BaseKeyFuncs<UStreamableRenderAsset*, const UStreamableRenderAsset*, false>
		{
			static KeyInitType	GetSetKey(ElementInitType& Element) { return Element; }
			static KeyInitType	GetSetKey(const TObjectPtr<UStreamableRenderAsset>& Element) { return Element.Get(); }			
			static bool			Matches(KeyInitType A, KeyInitType B) { return A == B; }
			static uint32		GetKeyHash(KeyInitType Key) { return GetTypeHash(Key); }
		};

		/** All the textures/meshes referenced in StreamingRenderAssets. Used to handle deleted textures/meshes. */
		TSet<TObjectPtr<UStreamableRenderAsset>, FPointerKeyFunc> ReferencedRenderAssets;

		/** All the currently installed optional bulkdata files */
		TSet<FString> OptionalBulkDataFiles;

		/** Index of the StreamingTexture that will be updated next by UpdateStreamingRenderAssets(). */
		int32 CurrentUpdateStreamingRenderAssetIndex;
//END: Thread-safe functions and data

	void	SetLastUpdateTime();
	void	UpdateStats();
	void	UpdateCSVOnlyStats();
	void	LogViewLocationChange();

	void	IncrementalUpdate( float Percentage, bool bUpdateDynamicComponents);

	/**
	 * Update all pending states.
	 *
	 * @param bUpdateDynamicComponents		Whether dynamic component state should also be updated.
	 */
	void	UpdatePendingStates(bool bUpdateDynamicComponents);

	/**
	 * Complete all pending async work and complete all pending state updates.
	 *
	 * @param bCompleteFullUpdateCycle		Whether to complete the full update cycle usually spread accross several frames.
	 */
	void	SyncStates(bool bCompleteFullUpdateCycle);

	/**
	 * Called on game thread when no new elements are added to the StreamingRenderAssets array and
	 * the elements in the array are not removed or reordered. This runs in parrallel with the
	 * async update task so the streaming meta data in each FStreamingRenderAsset can change
	 */
	void ProcessPendingMipCopyRequests();

	/**
	 * Mip-change callbacks can only be called on the game thread.
	 * When asset streamign status is updated on an async task, we need to tick their callabcks later.
	 */
	void TickDeferredMipLevelChangeCallbacks();

	void ProcessPendingLevelManagers();

	/** Cached from the system settings. */
	int32 NumStreamedMips_Texture[TEXTUREGROUP_MAX];
	TArray<int32> NumStreamedMips_StaticMesh;
	TArray<int32> NumStreamedMips_SkeletalMesh;

	FRenderAssetStreamingSettings Settings;

	/** Async work for calculating priorities and target number of mips for all textures/meshes. */
	FAsyncTask<FRenderAssetStreamingMipCalcTask>*	AsyncWork;

	/** Async work for render asset instance managers. */
	TRefCountPtr<RenderAssetInstanceTask::FDoWorkAsyncTask> RenderAssetInstanceAsyncWork;

	/** Texture/mesh instances from dynamic primitives. Owns the data for all levels. */
	FDynamicRenderAssetInstanceManager DynamicComponentManager;

	/** New textures/meshes, before they've been added to the thread-safe container. */
	TArray<UStreamableRenderAsset*>	PendingStreamingRenderAssets;

	/** The list of indices with null render asset in StreamingRenderAssets. */
	TArray<int32>	RemovedRenderAssetIndices;

	/** [Game/Task Thread] A list of assets whose callbacks need to be ticked on the game thread. */
	TArray<UStreamableRenderAsset*> DeferredTickCBAssets;

	/** [Game Thread] Forced fully resident assets that need to be loaded ASAP when visible. */
	TSet<UStreamableRenderAsset*> FastResponseRenderAssets;

	/** [Game Thread] Fast response assets detected visible before the end of full streaming update. */
	TSet<int32> VisibleFastResponseRenderAssetIndices;

	// Represent a pending request to stream in/out mips to/from GPU for a texture or mesh
	struct FPendingMipCopyRequest
	{
		const UStreamableRenderAsset* RenderAsset;
		// Used to find the corresponding FStreamingRenderAsset in the StreamingRenderAssets array
		int32 CachedIdx;

		FPendingMipCopyRequest() = default;

		FPendingMipCopyRequest(const UStreamableRenderAsset* InAsset, int32 InCachedIdx)
			: RenderAsset(InAsset)
			, CachedIdx(InCachedIdx)
		{}
	};
	TArray<FPendingMipCopyRequest> PendingMipCopyRequests;

	// The index of the next FPendingMipCopyRequest to process so the requests can be amortized accross multiple frames
	int32 CurrentPendingMipCopyRequestIdx;

	/** Level data */
	TArray<FLevelRenderAssetManager*> LevelRenderAssetManagers;

	// Used to prevent hazard when LevelRenderAssetManagers array is modified through recursion
	struct FScopedLevelRenderAssetManagersLock
	{
		FRenderAssetStreamingManager* StreamingManager;
		TArray<FLevelRenderAssetManager*> PendingAddLevelManagers;
		TArray<FLevelRenderAssetManager*> PendingRemoveLevelManagers;

		FScopedLevelRenderAssetManagersLock(FRenderAssetStreamingManager* InStreamingManager);

		~FScopedLevelRenderAssetManagersLock();
	};

	friend struct FScopedLevelRenderAssetManagersLock;

	FScopedLevelRenderAssetManagersLock* LevelRenderAssetManagersLock;

	/** Stages [0,N-2] is non-threaded data collection, Stage N-1 is wait-for-AsyncWork-and-finalize. */
	int32					ProcessingStage;

	/** Total number of processing stages (N). */
	int32					NumRenderAssetProcessingStages;

	/** Whether to support texture/mesh instance streaming for dynamic (movable/spawned) objects. */
	bool					bUseDynamicStreaming;

	float					BoostPlayerTextures;

	/** Amount of memory to leave free in the render asset pool. */
	int64					MemoryMargin;

	/** The actual memory pool size available to stream textures/meshes, excludes non-streaming texture/mesh, temp memory (for streaming mips), memory margin (allocator overhead). */
	int64					EffectiveStreamingPoolSize;

	// Stats we need to keep across frames as we only iterate over a subset of textures.

	int64 MemoryOverBudget;
	int64 MaxEverRequired;

	/** Whether render asset streaming is paused or not. When paused, it won't stream any textures/meshes in or out. */
	bool bPauseRenderAssetStreaming;

	/** Last time all data were fully updated. Instances are considered visible if they were rendered between that last time and the current time. */
	float LastWorldUpdateTime;

	/** LastWorldUpdateTime seen by the async task. */
	float LastWorldUpdateTime_MipCalcTask;

	FRenderAssetStreamingStats DisplayedStats;
	FRenderAssetStreamingStats GatheredStats;

	TArray<int32> InflightRenderAssets;

	/** A critical section to handle concurrent access on MountedStateDirtyFiles. */
	FCriticalSection MountedStateDirtyFilesCS;
	/** Whether all file mounted states needs to be re-evaluated. */
	bool bRecacheAllFiles = false;
	/** The file hashes which mounted state needs to be re-evaluated. */
	typedef TSet<FIoFilenameHash> FIoFilenameHashSet;
	FIoFilenameHashSet MountedStateDirtyFiles;

	FDelegateHandle OnPakFileMounted2Handle;
	FDelegateHandle NewFileAddedDelegateHandle;

	ENGINE_API virtual void MarkMountedStateDirty(FIoFilenameHash FilenameHash) override;

	// A critical section use around code that could be called in parallel with NotifyPrimitiveUpdated() or NotifyPrimitiveUpdated_Concurrent().
	FTransactionallySafeCriticalSection CriticalSection;

	// An event used to prevent FUpdateStreamingRenderAssetsTask from overlapping with related work
	FGraphEventRef StreamingRenderAssetsSyncEvent;
	TArray<FStreamingRenderAsset>& GetStreamingRenderAssetsAsyncSafe();


	 // Assets temporarily protected from StreamOut 
	TArray<UStreamableRenderAsset*> StreamOutProtectedAssets;
	mutable FRWLock StreamOutProtectedAssetsLock;

	FAbandonedRenderAssetUpdateManager AbandonedRenderAssetUpdateManager;

	friend bool TrackRenderAssetEvent( FStreamingRenderAsset* StreamingRenderAsset, UStreamableRenderAsset* RenderAsset, bool bForceMipLevelsToBeResident, const FRenderAssetStreamingManager* Manager);
};

struct FRenderAssetStreamingManagerHelper
{
	static float GetWorldTime(UWorld* InWorld);
};
