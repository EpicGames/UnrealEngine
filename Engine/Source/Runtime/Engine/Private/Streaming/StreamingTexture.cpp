// Copyright Epic Games, Inc. All Rights Reserved.

#include "Streaming/StreamingTexture.h"
#include "Streaming/SimpleStreamableAssetManager.h"
#include "Streaming/StreamingManagerTexture.h"
#include "Engine/StaticMesh.h"
#include "Engine/SkeletalMesh.h"
#include "Engine/SkinnedAssetCommon.h"
#include "StaticMeshResources.h"

#if PLATFORM_DESKTOP
ENGINE_API int32 GUseMobileLODBiasOnDesktopES31 = 1;
static FAutoConsoleVariableRef CVarUseMobileLODBiasOnDesktopES31(
	TEXT("r.Streaming.UseMobileLODBiasOnDesktopES31"),
	GUseMobileLODBiasOnDesktopES31,
	TEXT("If set apply mobile Min LOD bias on desktop platforms when running in ES31 mode")
);
#endif

static int32 GDefaultNoRefBias = 0;
static FAutoConsoleVariableRef CVarDefaultNoRefBias(
	TEXT("r.Streaming.DefaultNoRefLODBias"),
	GDefaultNoRefBias,
	TEXT("The default LOD bias for no-ref meshes"),
	ECVF_Scalability);

FStreamingRenderAsset::FStreamingRenderAsset(
	UStreamableRenderAsset* InRenderAsset,
	TConstArrayView<int32> NumStreamedMips,
	const FRenderAssetStreamingSettings& Settings)
	: RenderAsset(InRenderAsset)
	, SSAMAssetHandle(InRenderAsset->GetSSAMAssetHandle())
	, RenderAssetType(InRenderAsset->GetRenderAssetType())
{
	UpdateStaticData(Settings);
	UpdateDynamicData(NumStreamedMips, Settings);

	// In the editor, some paths can recreate the FStreamingRenderAsset, which could potentiallly trigger the unkown ref heuristic.
	// To prevent this, we consider that the asset bindings where reset when creating the FStreamingRenderAsset.
	// In game, we set it to FLT_MAX so that unkown ref heurisitic can kick in immeditaly (otherwise it incurs a 5 sec penalty on async loading)
	InstanceRemovedTimestamp = GIsEditor ? FApp::GetCurrentTime() : -FLT_MAX;
	DynamicBoostFactor = 1.f;

	SetFlag(FStreamingRenderAssetFlags::HasUpdatePending, InRenderAsset && InRenderAsset->bHasStreamingUpdatePending);
	SetFlag(FStreamingRenderAssetFlags::WasSSAMRegistered, FSimpleStreamableAssetManager::IsAssetRegistered(SSAMAssetHandle));

	SetFlag(FStreamingRenderAssetFlags::ForceFullyLoadHeuristic, false);
	SetFlag(FStreamingRenderAssetFlags::UseUnkownRefHeuristic, false);
	NumMissingMips = 0;
	SetFlag(FStreamingRenderAssetFlags::LooksLowRes, false);
	SetFlag(FStreamingRenderAssetFlags::MissingTooManyMips, false);
	VisibleWantedMips = MinAllowedMips;
	HiddenWantedMips = MinAllowedMips;
	RetentionPriority = 0;
	NormalizedScreenSize = 0.f;
	BudgetedMips = MinAllowedMips;
	NumForcedMips = 0;
	LoadOrderPriority = 0;
	WantedMips = MinAllowedMips;
}

void FStreamingRenderAsset::UpdateStaticData(const FRenderAssetStreamingSettings& Settings)
{
	FMemory::Memzero(CumulativeLODSizes);

	if (RenderAsset)
	{
		const FStreamableRenderResourceState ResourceState = RenderAsset->GetStreamableResourceState();

		LODGroup = RenderAsset->GetLODGroupForStreaming();
		BudgetMipBias = 0;

		if (IsTexture())
		{
			if (!ensureMsgf(FMath::IsWithin(LODGroup, 0, (int32)TEXTUREGROUP_MAX), TEXT("Invalid LODGroup %d for %s"), LODGroup, *RenderAsset->GetName()))
			{
				LODGroup = 0;
			}

			check(ResourceState.MaxNumLODs <= UE_ARRAY_COUNT(CumulativeLODSizes));
			const TextureGroup TextureLODGroup = static_cast<TextureGroup>(LODGroup);
			BoostFactor = GetExtraBoost(TextureLODGroup, Settings);
			SetFlag(FStreamingRenderAssetFlags::LoadWithHigherPriority, Settings.HighPriorityLoad_Texture[LODGroup]);
			SetFlag(FStreamingRenderAssetFlags::IsTerrainTexture, (TextureLODGroup == TEXTUREGROUP_Terrain_Heightmap || TextureLODGroup == TEXTUREGROUP_Terrain_Weightmap));
			SetFlag(FStreamingRenderAssetFlags::IsVirtualTexture, ResourceState.bSupportsVirtualStreaming);
		}
		else
		{
			check(ResourceState.MaxNumLODs <= UE_ARRAY_COUNT(CumulativeLODSizes_Mesh));
			check(ResourceState.MaxNumLODs <= UE_ARRAY_COUNT(LODScreenSizes));

			// Default boost value .71 is too small for meshes
			BoostFactor = 1.f;
			SetFlag(FStreamingRenderAssetFlags::LoadWithHigherPriority, false);
			SetFlag(FStreamingRenderAssetFlags::IsTerrainTexture, false);
			SetFlag(FStreamingRenderAssetFlags::IsVirtualTexture, false);
			if (RenderAssetType == EStreamableRenderAssetType::StaticMesh)
			{
				const UStaticMesh* StaticMesh = CastChecked<UStaticMesh>(RenderAsset);
				for (int32 LODIndex = 0; LODIndex < ResourceState.MaxNumLODs; ++LODIndex)
				{
					check(LODIndex < UE_ARRAY_COUNT(LODScreenSizes)); // See ScreenSize.
					// Screen sizes stored on assets are 2R/D where R is the radius of bounding spheres and D is the
					// distance from view origins to bounds origins. The factor calculated by the streamer, however,
					// is R/D so multiply 0.5 here
					LODScreenSizes[ResourceState.MaxNumLODs - LODIndex - 1] = StaticMesh->GetRenderData()->ScreenSize[LODIndex + ResourceState.AssetLODBias].GetValue() * 0.5f;
				}
			}
			else if (RenderAssetType == EStreamableRenderAssetType::SkeletalMesh)
			{
				USkeletalMesh* SkeletalMesh = CastChecked<USkeletalMesh>(RenderAsset);
				for (int32 LODIndex = 0; LODIndex < ResourceState.MaxNumLODs; ++LODIndex)
				{
					const FSkeletalMeshLODInfo* AssetLODInfo = SkeletalMesh->GetLODInfo(LODIndex + ResourceState.AssetLODBias);
					LODScreenSizes[ResourceState.MaxNumLODs - LODIndex - 1] = AssetLODInfo->ScreenSize.GetValue() * 0.5f;
				}
			}
		}

		for (int32 LODIndex = 0; LODIndex < ResourceState.MaxNumLODs; ++LODIndex)
		{
			CumulativeLODSizes[LODIndex] = RenderAsset->CalcCumulativeLODSize(LODIndex + 1);
		}

		// If there are optional mips
		if (ResourceState.NumNonOptionalLODs < ResourceState.MaxNumLODs)
		{
			// Use the hash for the smallest asset index (highest index) since this LOD is always included in optional mip load.
			OptionalMipsState = EOptionalMipsState::OMS_NotCached;
			OptionalFileHash = RenderAsset->GetMipIoFilenameHash(ResourceState.LODCountToAssetFirstLODIdx(ResourceState.NumNonOptionalLODs + 1));
		}
		else
		{
			OptionalMipsState = EOptionalMipsState::OMS_NoOptionalMips;
			OptionalFileHash = INVALID_IO_FILENAME_HASH;
		}
	}
	else
	{
		LODGroup = TEXTUREGROUP_World;
		RenderAssetType = EStreamableRenderAssetType::None;
		BudgetMipBias = 0;
		BoostFactor = 1.f;
		OptionalMipsState = EOptionalMipsState::OMS_NoOptionalMips;
		OptionalFileHash = INVALID_IO_FILENAME_HASH;

		SetFlag(FStreamingRenderAssetFlags::LoadWithHigherPriority, false);
		SetFlag(FStreamingRenderAssetFlags::IsTerrainTexture, false);
	}
}

void FStreamingRenderAsset::UpdateOptionalMipsState_Async()
{
	// Cache the pointer to prevent a race condition with FRenderAssetStreamingManager::RemoveStreamingRenderAsset()
	UStreamableRenderAsset*	CachedRenderAsset = RenderAsset;
	if (CachedRenderAsset && OptionalMipsState == EOptionalMipsState::OMS_NotCached && OptionalFileHash != INVALID_IO_FILENAME_HASH)
	{
		FStreamableRenderResourceState ResourceState = CachedRenderAsset->GetStreamableResourceState();
		if (ResourceState.IsValid() && CachedRenderAsset->DoesMipDataExist(ResourceState.AssetLODBias))
		{
			OptionalMipsState = EOptionalMipsState::OMS_HasOptionalMips;
		}
		else
		{
			OptionalMipsState = EOptionalMipsState::OMS_NoOptionalMips;
		}
	}	
}

void FStreamingRenderAsset::UpdateDynamicData(TConstArrayView<int32> NumStreamedMips, const FRenderAssetStreamingSettings& Settings, TArray<UStreamableRenderAsset*>* DeferredTickCBAssets)
{
	// Note that those values are read from the async task and must not be assigned temporary values!!
	if (RenderAsset)
	{
		// Get the resource state after calling UpdateStreamingStatus() since it might have updated it.
		const FStreamableRenderResourceState ResourceState = UpdateStreamingStatus(DeferredTickCBAssets);

		// The last render time of this texture/mesh. Can be FLT_MAX when texture has no resource.
		const float LastRenderTimeForTexture = RenderAsset->GetLastRenderTimeForStreaming();
		LastRenderTime = (FApp::GetCurrentTime() > LastRenderTimeForTexture) ? float( FApp::GetCurrentTime() - LastRenderTimeForTexture ) : 0.0f;

		const bool bForceFullyLoadFlag = RenderAsset->ShouldMipLevelsBeForcedResident();
		SetFlag(FStreamingRenderAssetFlags::ForceFullyLoad, bForceFullyLoadFlag);

		SetFlag(FStreamingRenderAssetFlags::IgnoreStreamingMipBias, RenderAsset->bIgnoreStreamingMipBias);

		const int32 NumCinematicMipLevels = (bForceFullyLoadFlag && RenderAsset->bUseCinematicMipLevels) ? RenderAsset->NumCinematicMipLevels : 0;

		int32 LODBias = 0;
		if (!Settings.bUseAllMips)
		{
			const int32 ResourceLODBias = FMath::Max<int32>(0, RenderAsset->GetCachedLODBias() - ResourceState.AssetLODBias);
			LODBias = FMath::Max<int32>(ResourceLODBias - NumCinematicMipLevels, 0);

			// Reduce the max allowed resolution according to LODBias if the texture group allows it.
			if (IsMaxResolutionAffectedByGlobalBias() && !Settings.bUsePerTextureBias)
			{
				LODBias += Settings.GlobalMipBias;
			}

			LODBias += BudgetMipBias;

#if PLATFORM_DESKTOP
			if (GUseMobileLODBiasOnDesktopES31 != 0 && GMaxRHIFeatureLevel == ERHIFeatureLevel::ES3_1)
			{
				LODBias += ResourceState.LODBiasModifier;
			}
#endif
		}

		int32 LocalNoRefBias = 0;
		if (RenderAssetType == EStreamableRenderAssetType::StaticMesh || RenderAssetType == EStreamableRenderAssetType::SkeletalMesh)
		{
			LocalNoRefBias = RenderAsset->GetCurrentNoRefStreamingLODBias();
			if (LocalNoRefBias < 0)
			{
				LocalNoRefBias = GDefaultNoRefBias;
			}
		}
		NoRefLODBias = LocalNoRefBias;

		check( ResourceState.MaxNumLODs >= ResourceState.NumNonOptionalLODs );
		check( ResourceState.NumNonOptionalLODs >= ResourceState.NumNonStreamingLODs );

		// If the optional mips are not available, or if we shouldn't load them now, clamp the possible mips requested. 
		// (when the non-optional mips are not yet loaded, loading optional mips generates cross files requests).
		// This is not bullet proof though since the texture/mesh could have a pending stream-out request.
		if (OptionalMipsState != EOptionalMipsState::OMS_HasOptionalMips || ResidentMips < ResourceState.NumNonOptionalLODs)
		{
			MaxAllowedMips = FMath::Clamp<int32>(ResourceState.MaxNumLODs - LODBias, ResourceState.NumNonStreamingLODs, ResourceState.NumNonOptionalLODs);
		}
		else
		{
			MaxAllowedMips = FMath::Clamp<int32>(ResourceState.MaxNumLODs - LODBias, ResourceState.NumNonStreamingLODs, ResourceState.MaxNumLODs);
		}
	
		check( MaxAllowedMips >= ResourceState.NumNonStreamingLODs );

		check(LODGroup < NumStreamedMips.Num());
		if (NumStreamedMips[LODGroup] > 0)
		{
			MinAllowedMips = FMath::Clamp<int32>(ResourceState.MaxNumLODs - NumStreamedMips[LODGroup], ResourceState.NumNonStreamingLODs, MaxAllowedMips);
		}
		else
		{
			MinAllowedMips = ResourceState.NumNonStreamingLODs;
		}
	}
	else
	{
		SetFlag(FStreamingRenderAssetFlags::ForceFullyLoad, false);
		SetFlag(FStreamingRenderAssetFlags::IgnoreStreamingMipBias, false);
		ResidentMips = 0;
		RequestedMips = 0;
		MinAllowedMips = 0;
		MaxAllowedMips = 0;
		NoRefLODBias = 0;
		OptionalMipsState = EOptionalMipsState::OMS_NotCached;
		LastRenderTime = FLT_MAX;	
	}
}

FStreamableRenderResourceState FStreamingRenderAsset::UpdateStreamingStatus(TArray<UStreamableRenderAsset*>* DeferredTickCBAssets)
{
	FStreamableRenderResourceState ResourceState;

	if (RenderAsset)
	{
		RenderAsset->TickStreaming(UStreamableRenderAsset::ETickStreamingFlags::SendCompletionEvents, DeferredTickCBAssets);

		// Call only after UpdateStreamingStatus() since it could update it.
		ResourceState = RenderAsset->GetStreamableResourceState();

		// This must be updated after UpdateStreamingStatus
		ResidentMips = ResourceState.NumResidentLODs;
		RequestedMips = ResourceState.NumRequestedLODs;
	}
	return ResourceState;
}

float FStreamingRenderAsset::GetExtraBoost(TextureGroup	LODGroup, const FRenderAssetStreamingSettings& Settings)
{
	const float DistanceScale = GetDefaultExtraBoost(Settings.bUseNewMetrics);

	if (LODGroup == TEXTUREGROUP_Terrain_Heightmap || LODGroup == TEXTUREGROUP_Terrain_Weightmap) 
	{
		// Terrain are not affected by any kind of scale. Important since instance can use hardcoded resolution.
		// Used the Distance Scale from the new metrics is not big enough to affect which mip gets selected.
		return DistanceScale;
	}
	else if (LODGroup == TEXTUREGROUP_Lightmap)
	{
		return FMath::Min<float>(DistanceScale, GLightmapStreamingFactor);
	}
	else if (LODGroup == TEXTUREGROUP_Shadowmap)
	{
		return FMath::Min<float>(DistanceScale, GShadowmapStreamingFactor);
	}
	else
	{
		return DistanceScale;
	}
}

int32 FStreamingRenderAsset::GetWantedMipsFromSize(float Size, float InvMaxScreenSizeOverAllViews) const
{
	if (IsTexture())
	{
		float WantedMipsFloat = 1.0f + FMath::Log2(FMath::Max(1.f, Size));
		int32 WantedMipsInt = FMath::CeilToInt(WantedMipsFloat);
		return FMath::Clamp<int32>(WantedMipsInt, MinAllowedMips, MaxAllowedMips);
	}
	else
	{
		check(RenderAssetType == EStreamableRenderAssetType::StaticMesh || RenderAssetType == EStreamableRenderAssetType::SkeletalMesh);
		if (Size == FLT_MAX)
		{
			return MaxAllowedMips;
		}
		int32 Result = MaxAllowedMips;
		const float NormalizedSize = Size * InvMaxScreenSizeOverAllViews;
		for (int32 NumMips = MinAllowedMips; NumMips <= MaxAllowedMips; ++NumMips)
		{
			if (GetNormalizedScreenSize(NumMips) >= NormalizedSize)
			{
				Result = NumMips;
				break;
			}
		}
		return GetFlag(FStreamingRenderAssetFlags::UseUnkownRefHeuristic) ? FMath::Max(Result - NoRefLODBias, MinAllowedMips) : Result;;
	}
}

/** Set the wanted mips from the async task data */
void FStreamingRenderAsset::SetPerfectWantedMips_Async(
	float MaxSize,
	float MaxSize_VisibleOnly,
	float MaxScreenSizeOverAllViews,
	int32 MaxNumForcedLODs,
	bool InLooksLowRes,
	const FRenderAssetStreamingSettings& Settings)
{
	SetFlag(FStreamingRenderAssetFlags::ForceFullyLoadHeuristic, (MaxSize == FLT_MAX || MaxSize_VisibleOnly == FLT_MAX));
	SetFlag(FStreamingRenderAssetFlags::LooksLowRes, InLooksLowRes); // Things like lightmaps, HLOD and close instances.
	NormalizedScreenSize = 0.f;

	if (MaxNumForcedLODs >= MaxAllowedMips)
	{
		VisibleWantedMips = HiddenWantedMips = NumForcedMips = MaxAllowedMips;
		NumMissingMips = 0;
		return;
	}

	float InvMaxScreenSizeOverAllViews = 1.f;
	if (IsMesh())
	{
		InvMaxScreenSizeOverAllViews = 1.f / MaxScreenSizeOverAllViews;
		NormalizedScreenSize = FMath::Max(MaxSize, MaxSize_VisibleOnly) * InvMaxScreenSizeOverAllViews;
	}

	NumForcedMips = FMath::Min(MaxNumForcedLODs, MaxAllowedMips);
	VisibleWantedMips = FMath::Max(GetWantedMipsFromSize(MaxSize_VisibleOnly, InvMaxScreenSizeOverAllViews), NumForcedMips);

	// Terrain, Forced Fully Load and Things that already look bad are not affected by hidden scale.
	if (GetFlag(FStreamingRenderAssetFlags::IsTerrainTexture) || GetFlag(FStreamingRenderAssetFlags::ForceFullyLoadHeuristic) || GetFlag(FStreamingRenderAssetFlags::LooksLowRes))
	{
		HiddenWantedMips = FMath::Max(GetWantedMipsFromSize(MaxSize, InvMaxScreenSizeOverAllViews), NumForcedMips);
		NumMissingMips = 0; // No impact for terrains as there are not allowed to drop mips.
	}
	else
	{
		HiddenWantedMips = FMath::Max(GetWantedMipsFromSize(MaxSize * Settings.HiddenPrimitiveScale, InvMaxScreenSizeOverAllViews), NumForcedMips);
		// NumMissingMips contains the number of mips not loaded because of HiddenPrimitiveScale. When out of budget, those texture will be considered as already sacrificed.
		NumMissingMips = FMath::Max<int32>(GetWantedMipsFromSize(MaxSize, InvMaxScreenSizeOverAllViews) - FMath::Max<int32>(VisibleWantedMips, HiddenWantedMips), 0);
	}
}

/**
 * Once the wanted mips are computed, the async task will check if everything fits in the budget.
 *  This only consider the highest mip that will be requested eventually, so that slip requests are stables.
 */
int64 FStreamingRenderAsset::UpdateRetentionPriority_Async(bool bPrioritizeMesh)
{
	// Reserve the budget for the max mip that will be loaded eventually (ignore the effect of split requests)
	BudgetedMips = GetPerfectWantedMips();
	RetentionPriority = 0;

	if (RenderAsset)
	{
		const bool bIsHuge    = GetSize(BudgetedMips) >= 8 * 1024 * 1024 && LODGroup != TEXTUREGROUP_Lightmap && LODGroup != TEXTUREGROUP_Shadowmap;
		const bool bIsTerrainTextureFlag = GetFlag(FStreamingRenderAssetFlags::IsTerrainTexture);
		const bool bForceFullyLoadHeuristicFlag = GetFlag(FStreamingRenderAssetFlags::ForceFullyLoadHeuristic);
		const bool bLooksLowResFlag = GetFlag(FStreamingRenderAssetFlags::LooksLowRes);
		const bool bShouldKeep = bIsTerrainTextureFlag || bForceFullyLoadHeuristicFlag || (bLooksLowResFlag && !bIsHuge);
		const bool bIsSmall   = GetSize(BudgetedMips) <= 200 * 1024; 
		const bool bIsVisible = VisibleWantedMips >= HiddenWantedMips; // Whether the first mip dropped would be a visible mip or not.

		// Here we try to have a minimal amount of priority flags for last render time to be meaningless.
		// We mostly want thing not seen from a long time to go first to avoid repeating load / unload patterns.

		if (bPrioritizeMesh && IsMesh())		RetentionPriority += 4096; // Only consider meshes after textures are processed for faster metric calculation.
		if (bShouldKeep)						RetentionPriority += 2048; // Keep forced fully load as much as possible.
		if (bIsVisible)							RetentionPriority += 1024; // Keep visible things as much as possible.
		if (!bIsHuge)							RetentionPriority += 512; // Drop high resolution which usually target ultra close range quality.
		if (GetFlag(FStreamingRenderAssetFlags::LoadWithHigherPriority) || bIsSmall)	RetentionPriority += 256; // Try to keep character of small texture as they don't pay off.
		if (!bIsVisible)						RetentionPriority += FMath::Clamp<int32>(255 - (int32)LastRenderTime, 1, 255); // Keep last visible first.

		return GetSize(BudgetedMips);
	}
	else
	{
		return 0;
	}
}

int32 FStreamingRenderAsset::ClampMaxResChange_Internal(int32 NumMipDropRequested) const
{
	// Don't drop bellow min allowed mips. Also ensure that MinAllowedMips < MaxAllowedMips in order allow the BudgetMipBias to reset.
	return FMath::Min(MaxAllowedMips - MinAllowedMips - 1, NumMipDropRequested);
}

int64 FStreamingRenderAsset::DropMaxResolution_Async(int32 NumDroppedMips)
{
	if (RenderAsset)
	{
		NumDroppedMips = ClampMaxResChange_Internal(NumDroppedMips);

		if (NumDroppedMips > 0)
		{
			// Decrease MaxAllowedMips and increase BudgetMipBias (as it should include it)
			MaxAllowedMips -= NumDroppedMips;
			BudgetMipBias += NumDroppedMips;

			if (BudgetedMips > MaxAllowedMips)
			{
				const int64 FreedMemory = GetSize(BudgetedMips) - GetSize(MaxAllowedMips);

				BudgetedMips = MaxAllowedMips;
				VisibleWantedMips = FMath::Min<int32>(VisibleWantedMips, MaxAllowedMips);
				HiddenWantedMips = FMath::Min<int32>(HiddenWantedMips, MaxAllowedMips);

				return FreedMemory;
			}
		}
		else // If we can't reduce resolution, still drop a mip if possible to free memory (eventhough it won't be persistent)
		{
			return DropOneMip_Async();
		}
	}
	return 0;
}

int64 FStreamingRenderAsset::DropOneMip_Async()
{
	if (RenderAsset && BudgetedMips > MinAllowedMips)
	{
		--BudgetedMips;
		return GetSize(BudgetedMips + 1) - GetSize(BudgetedMips);
	}
	else
	{
		return 0;
	}
}

int64 FStreamingRenderAsset::KeepOneMip_Async()
{
	if (RenderAsset && BudgetedMips < FMath::Min<int32>(ResidentMips, MaxAllowedMips))
	{
		++BudgetedMips;
		return GetSize(BudgetedMips) - GetSize(BudgetedMips - 1);
	}
	else
	{
		return 0;
	}
}

int64 FStreamingRenderAsset::GetDropMaxResMemDelta(int32 NumDroppedMips) const
{
	if (RenderAsset)
	{
		NumDroppedMips = ClampMaxResChange_Internal(NumDroppedMips);
		return GetSize(MaxAllowedMips) - GetSize(MaxAllowedMips - NumDroppedMips);
	}

	return 0;
}

int64 FStreamingRenderAsset::GetDropOneMipMemDelta() const
{
	return GetSize(BudgetedMips + 1) - GetSize(BudgetedMips);
}

bool FStreamingRenderAsset::UpdateLoadOrderPriority_Async(const FRenderAssetStreamingSettings& Settings)
{
	LoadOrderPriority = 0;
	SetFlag(FStreamingRenderAssetFlags::MissingTooManyMips, false);

	const bool bIsTerrainTextureFlag = GetFlag(FStreamingRenderAssetFlags::IsTerrainTexture);

	// First load the visible mips, then later load the non visible part (does not apply to terrain textures as distance fields update may be waiting).
	if (ResidentMips < VisibleWantedMips && VisibleWantedMips < BudgetedMips && BudgetedMips >= Settings.MinMipForSplitRequest && !bIsTerrainTextureFlag)
	{
		WantedMips = VisibleWantedMips;
	}
	else
	{
		WantedMips = BudgetedMips;
	}

	// If the entry is valid and we need to send a new request to load/drop the right mip.
	if (RenderAsset && WantedMips != RequestedMips)
	{
		const bool bIsVisible			= ResidentMips < VisibleWantedMips; // Otherwise it means we are loading mips that are only useful for non visible primitives.
		const bool bMustLoadFirst		= GetFlag(FStreamingRenderAssetFlags::ForceFullyLoadHeuristic) || bIsTerrainTextureFlag || GetFlag(FStreamingRenderAssetFlags::LoadWithHigherPriority);

		if (WantedMips > RequestedMips)
		{
			const bool bMipIsImportant = WantedMips - ResidentMips > (GetFlag(FStreamingRenderAssetFlags::LooksLowRes) || IsMesh() ? 1 : 2);

			// Only consider visible assets to reduce the number of high priority requests
			SetFlag(FStreamingRenderAssetFlags::MissingTooManyMips, bIsVisible && bMipIsImportant && Settings.LowResHandlingMode != FRenderAssetStreamingSettings::LRHM_DoNothing);

			if (bIsVisible)				LoadOrderPriority += 1024;
			if (bMustLoadFirst)			LoadOrderPriority += 512;
			if (bMipIsImportant)		LoadOrderPriority += 256;
			if (!bIsVisible)			LoadOrderPriority += FMath::Clamp<int32>(255 - (int32)LastRenderTime, 1, 255);
		}
		else // WantedMips < RequestedMips
		{
			// For stream out operations, unload mips that we will be less important to load quickly.
			// Visibility is less relevant here because stream in requests might come from a visibily change.
			if (!bMustLoadFirst)		LoadOrderPriority += 1024;
			if (!bIsVisible)			LoadOrderPriority += 512;
		}

		return true;
	}
	else
	{
		return false;
	}
}

void FStreamingRenderAsset::CancelStreamingRequest()
{
	if (RenderAsset)
	{
		RenderAsset->CancelPendingStreamingRequest();
		UpdateStreamingStatus();
	}
}

void FStreamingRenderAsset::StreamWantedMips(FRenderAssetStreamingManager& Manager)
{
	StreamWantedMips_Internal(Manager, false);
}

void FStreamingRenderAsset::CacheStreamingMetaData()
{
	SetFlag(FStreamingRenderAssetFlags::CachedForceFullyLoadHeuristic, GetFlag(FStreamingRenderAssetFlags::ForceFullyLoadHeuristic));
	CachedWantedMips = WantedMips;
	CachedVisibleWantedMips = VisibleWantedMips;
}

void FStreamingRenderAsset::StreamWantedMipsUsingCachedData(FRenderAssetStreamingManager& Manager)
{
	StreamWantedMips_Internal(Manager, true);
}

void FStreamingRenderAsset::StreamWantedMips_Internal(FRenderAssetStreamingManager& Manager, bool bUseCachedData)
{
	if (RenderAsset && !RenderAsset->HasPendingInitOrStreaming())
	{
		const FStreamableRenderResourceState ResourceState = RenderAsset->GetStreamableResourceState();

		const uint32 bLocalForceFullyLoadHeuristic = bUseCachedData ? GetFlag(FStreamingRenderAssetFlags::CachedForceFullyLoadHeuristic) : GetFlag(FStreamingRenderAssetFlags::ForceFullyLoadHeuristic);
		const int32 LocalVisibleWantedMips = bUseCachedData ? CachedVisibleWantedMips : VisibleWantedMips;
		// Update ResidentMips now as it is guarantied to not change here (since no pending requests).
		ResidentMips = ResourceState.NumResidentLODs;

		// Prevent streaming-in optional mips and non optional mips as they are from different files.
		int32 LocalWantedMips = bUseCachedData ? CachedWantedMips : WantedMips;
		if (ResidentMips < ResourceState.NumNonOptionalLODs && LocalWantedMips > ResourceState.NumNonOptionalLODs)
		{
			LocalWantedMips = ResourceState.NumNonOptionalLODs;
		}

#if !UE_BUILD_SHIPPING
		RenderAsset->CachedNumWantedLODs = static_cast<uint8>(LocalWantedMips);
#endif

		if (LocalWantedMips != ResidentMips)
		{
			if (LocalWantedMips < ResidentMips)
			{
				// Protection-aware StreamOut: if an external consumer holds the asset,
				// this returns false without invoking StreamOut. The async task will
				// keep producing stream-out requests so the next tick after release
				// picks this up.
				if (!Manager.StreamOutProtected(RenderAsset, LocalWantedMips))
				{
					return;
				}
			}
			else // WantedMips > ResidentMips
			{
				const bool bShouldPrioritizeAsyncIORequest = (bLocalForceFullyLoadHeuristic || GetFlag(FStreamingRenderAssetFlags::IsTerrainTexture) || GetFlag(FStreamingRenderAssetFlags::LoadWithHigherPriority) || IsMissingTooManyMips()) && LocalWantedMips <= LocalVisibleWantedMips;
				RenderAsset->StreamIn(LocalWantedMips, bShouldPrioritizeAsyncIORequest);
			}
			UpdateStreamingStatus();
			TrackRenderAssetEvent(this, RenderAsset, bLocalForceFullyLoadHeuristic != 0, &Manager);
		}
	}
}
