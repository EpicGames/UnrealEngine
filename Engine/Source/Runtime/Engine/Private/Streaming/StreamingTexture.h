// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/StreamableRenderAsset.h"
#include "Engine/TextureDefines.h"
#include "StaticMeshResources.h"
#include <atomic>

struct FRenderAssetStreamingManager;
struct FRenderAssetStreamingSettings;

/**
 * Atomic bitset backing the per-asset flags on FStreamingRenderAsset. The previous design used
 * plain ': 1' bitfields packed into shared bytes; concurrent writes from the game thread and the
 * streaming async task to different bits in the same byte would collide via read-modify-write
 * stores and could trash each other's updates. fetch_or / fetch_and on a single std::atomic<uint16>
 * give a true atomic single-bit update on every supported platform (LOCK OR on x86-64, LL/SC retry
 * loop on ARM64). Each bit is a stand-alone flag with no other data coordinated through it, so
 * std::memory_order_relaxed is sufficient; cross-thread happens-before is provided by the
 * task-graph sync event around FRenderAssetStreamingMipCalcTask.
 */
class FStreamingRenderAssetFlags
{
public:
	enum EBit : uint8
	{
		IsVirtualTexture              = 0,
		LoadWithHigherPriority        = 1,
		IsTerrainTexture              = 2,
		ForceFullyLoad                = 3,
		IgnoreStreamingMipBias        = 4,
		HasUpdatePending              = 5,
		ForceFullyLoadHeuristic       = 6,
		UseUnkownRefHeuristic         = 7,
		LooksLowRes                   = 8,
		MissingTooManyMips            = 9,
		CachedForceFullyLoadHeuristic = 10,
		WasSSAMRegistered             = 11,
	};

	FStreamingRenderAssetFlags() = default;
	FStreamingRenderAssetFlags(const FStreamingRenderAssetFlags& Other) noexcept : Storage(Other.Storage.load(std::memory_order_relaxed)) {}
	FStreamingRenderAssetFlags(FStreamingRenderAssetFlags&& Other) noexcept : Storage(Other.Storage.load(std::memory_order_relaxed)) {}

	FStreamingRenderAssetFlags& operator=(const FStreamingRenderAssetFlags& Other) noexcept { Storage.store(Other.Storage.load(std::memory_order_relaxed), std::memory_order_relaxed); return *this; }
	FStreamingRenderAssetFlags& operator=(FStreamingRenderAssetFlags&& Other) noexcept { Storage.store(Other.Storage.load(std::memory_order_relaxed), std::memory_order_relaxed); return *this; }

	bool Get(EBit Bit) const
	{
		return (Storage.load(std::memory_order_relaxed) & (uint16(1) << Bit)) != 0;
	}

	void Set(EBit Bit, bool Value)
	{
		const uint16 Mask = uint16(1) << Bit;
		if (Value)
		{
			Storage.fetch_or(Mask, std::memory_order_relaxed);
		}
		else
		{
			Storage.fetch_and(static_cast<uint16>(~Mask), std::memory_order_relaxed);
		}
	}

private:
	std::atomic<uint16> Storage{ 0 };
};

/*-----------------------------------------------------------------------------
	FStreamingRenderAsset, the streaming system's version of UStreamableRenderAsset.
-----------------------------------------------------------------------------*/

/** Self-contained structure to manage a streaming texture/mesh, possibly on a separate thread. */
struct FStreamingRenderAsset
{
	enum EOptionalMipsState : uint8
	{
		OMS_NotCached,
		OMS_NoOptionalMips,
		OMS_HasOptionalMips,
		OMS_Num
	};

	/** std::atomic<double> wrapper with copy/move semantics so the enclosing struct stays TArray-storable. */
	struct FAtomicTimestamp
	{
		std::atomic<double> Value{0.0};

		FAtomicTimestamp() = default;
		FAtomicTimestamp(double V) noexcept : Value(V) {}
		FAtomicTimestamp(const FAtomicTimestamp& Other) noexcept : Value(Other.Value.load(std::memory_order_relaxed)) {}
		FAtomicTimestamp(FAtomicTimestamp&& Other) noexcept : Value(Other.Value.load(std::memory_order_relaxed)) {}

		FAtomicTimestamp& operator=(double V) noexcept { Value.store(V, std::memory_order_relaxed); return *this; }
		FAtomicTimestamp& operator=(const FAtomicTimestamp& Other) noexcept { Value.store(Other.Value.load(std::memory_order_relaxed), std::memory_order_relaxed); return *this; }
		FAtomicTimestamp& operator=(FAtomicTimestamp&& Other) noexcept { Value.store(Other.Value.load(std::memory_order_relaxed), std::memory_order_relaxed); return *this; }

		operator double() const noexcept { return Value.load(std::memory_order_relaxed); }
	};

	FStreamingRenderAsset(
		UStreamableRenderAsset* InRenderAsset,
		TConstArrayView<int32> NumStreamedMips,
		const FRenderAssetStreamingSettings& Settings);

	/** Update data that should not change unless changing settings. */
	void UpdateStaticData(const FRenderAssetStreamingSettings& Settings);

	/** Update data that the engine could change through gameplay. */
	void UpdateDynamicData(TConstArrayView<int32> NumStreamedMips, const FRenderAssetStreamingSettings& Settings, TArray<UStreamableRenderAsset*>* DeferredTickCBAssets = nullptr);

	/** Lightweight version of UpdateDynamicData. */
	FStreamableRenderResourceState UpdateStreamingStatus(TArray<UStreamableRenderAsset*>* DeferredTickCBAssets = nullptr);

	/**
	 * Returns the amount of memory used by the texture/mesh given a specified number of mip-maps, in bytes.
	 *
	 * @param MipCount	Number of mip-maps to account for
	 * @return			Total amount of memory used for the specified mip-maps, in bytes
	 */
	int32 GetSize( int32 InMipCount ) const
	{
		check(InMipCount > 0);
		check((uint32)InMipCount <= (IsTexture() ? UE_ARRAY_COUNT(CumulativeLODSizes) : UE_ARRAY_COUNT(CumulativeLODSizes_Mesh)));
		return CumulativeLODSizes[InMipCount - 1];
	}

	static const TCHAR* GetStreamingAssetTypeStr(EStreamableRenderAssetType InAssetType)
	{
		switch (InAssetType)
		{
		case EStreamableRenderAssetType::Texture:
			return TEXT("Texture");
		case EStreamableRenderAssetType::StaticMesh:
			return TEXT("StaticMesh");
		case EStreamableRenderAssetType::SkeletalMesh:
			return TEXT("SkeletalMesh");
		default:
			return TEXT("Unkown");
		}
	}

	static float GetDefaultExtraBoost(bool bUseNewMetrics)
	{
		// When accurate distance computation, we need to relax the distance otherwise it gets too conservative. (ex 513 goes to 1024)
		return bUseNewMetrics ? .71f : 1.f;
	}

	static float GetExtraBoost(TextureGroup	LODGroup, const FRenderAssetStreamingSettings& Settings);

	int32 GetWantedMipsFromSize(float Size, float InvMaxScreenSizeOverAllViews) const;

	/** Set the wanted mips from the async task data */
	void SetPerfectWantedMips_Async(
		float MaxSize,
		float MaxSize_VisibleOnly,
		float MaxScreenSizeOverAllViews,
		int32 MaxNumForcedLODs,
		bool InLooksLowRes,
		const FRenderAssetStreamingSettings& Settings);

	/** Init BudgetedMip and update RetentionPriority. Returns the size that would be taken if all budgeted mips where loaded. */
	int64 UpdateRetentionPriority_Async(bool bPrioritizeMesh);

	/** Reduce the maximum allowed resolution by 1 mip. Return the size freed by doing so. */
	int64 DropMaxResolution_Async(int32 NumDroppedMips);

	/** Reduce BudgetedMip by 1 and return the size freed by doing so. */
	int64 DropOneMip_Async();

	/** Increase BudgetedMip by 1, up to resident mips, and return the size taken. */
	int64 KeepOneMip_Async();

	/** Return the memory delta in bytes caused by max resolution change. Actual memory reduction is smaller or equal. **/
	int64 GetDropMaxResMemDelta(int32 NumDroppedMips) const;

	/** Return the memory delta in bytes if a mip is successfully dropped. */
	int64 GetDropOneMipMemDelta() const;

	float GetMaxAllowedSize(float MaxScreenSizeOverAllViews) const
	{
		return IsTexture() ? (float)(0x1 << (MaxAllowedMips - 1)) : MaxScreenSizeOverAllViews;
	}

	float GetNormalizedScreenSize(int32 NumMips) const
	{
		check(IsMesh());
		check(NumMips > 0 && (uint32)NumMips <= UE_ARRAY_COUNT(LODScreenSizes));
		return LODScreenSizes[NumMips - 1];
	}

	float GetLODScreenSize(int32 NumMips, float MaxScreenSizeOverAllViews) const
	{
		return IsTexture() ?
			static_cast<float>(1 << (NumMips - 1)) :
			GetNormalizedScreenSize(NumMips) * MaxScreenSizeOverAllViews;
	}

	/** Init load order. Return wether this texture has any load/unload request */
	bool UpdateLoadOrderPriority_Async(const FRenderAssetStreamingSettings& Settings);

	void UpdateOptionalMipsState_Async();
	
	void CancelStreamingRequest();
	void StreamWantedMips(FRenderAssetStreamingManager& Manager);

	// Cache meta data (e.g. WantedMips) for StreamWantedMipsUsingCachedData to use later on
	void CacheStreamingMetaData();

	// Stream using the meta data produced by last run of FRenderAssetStreamingMipCalcTask.
	// This allows streaming to happen in parallel with the async update task
	void StreamWantedMipsUsingCachedData(FRenderAssetStreamingManager& Manager);

	bool IsTexture() const
	{
		return RenderAssetType == EStreamableRenderAssetType::Texture;
	}

	bool IsVirtualTexture() const
	{
		return RenderAssetType == EStreamableRenderAssetType::Texture && GetFlag(FStreamingRenderAssetFlags::IsVirtualTexture);
	}

	bool IsMesh() const
	{
		// FRenderAssetStreamingManager only handles textures and meshes currently
		return RenderAssetType != EStreamableRenderAssetType::Texture;
	}

	bool IsMissingTooManyMips() const
	{
		return GetFlag(FStreamingRenderAssetFlags::MissingTooManyMips);
	}

	bool GetFlag(FStreamingRenderAssetFlags::EBit Bit) const
	{
		return Flags.Get(Bit);
	}

	void SetFlag(FStreamingRenderAssetFlags::EBit Bit, bool Value)
	{
		Flags.Set(Bit, Value);
	}

	FORCEINLINE int32 GetPerfectWantedMips() const { return FMath::Max<int32>(VisibleWantedMips,  HiddenWantedMips); }

	// Whether this texture/mesh can be affected by Global Bias and Budget Bias per texture/mesh.
	// It controls if the texture/mesh resolution can be scarificed to fit into budget.
	FORCEINLINE bool IsMaxResolutionAffectedByGlobalBias() const
	{
		// In editor, forced stream in should never have reduced mips as they can be edited.
		return (IsMesh() || LODGroup != TEXTUREGROUP_HierarchicalLOD)
			&& !GetFlag(FStreamingRenderAssetFlags::IsTerrainTexture)
			&& !(GetFlag(FStreamingRenderAssetFlags::ForceFullyLoadHeuristic) && GetFlag(FStreamingRenderAssetFlags::IgnoreStreamingMipBias))
			&& !(GIsEditor && GetFlag(FStreamingRenderAssetFlags::ForceFullyLoadHeuristic));
	}

	FORCEINLINE bool HasUpdatePending(bool bIsStreamingPaused, bool bHasViewPoint) const
	{
		const bool bBudgetedMipsIsValid = bHasViewPoint || GetFlag(FStreamingRenderAssetFlags::ForceFullyLoadHeuristic); // Force fully load don't need any viewpoint info.
		// If paused, nothing will update anytime soon.
		// If more mips will be streamed in eventually, wait.
		// Otherwise, if the distance based computation had no viewpoint, wait.
		return !bIsStreamingPaused && (BudgetedMips > ResidentMips || !bBudgetedMipsIsValid);
	}

	/***************************************************************
	 * Member data categories:
	 * (1) Members initialized when this is constructed => NEVER CHANGES
	 * (2) Cached dynamic members that need constant update => UPDATES IN UpdateDynamicData()
	 * (3) Helper data set by the streamer to handle special cases => CHANGES ANYTIME (gamethread)
	 * (4) Data generated by the async task. CHANGES ANYTIME (taskthread)
	 * (5) Data cached to allow streaming to run in parallel with meta data update
	 ***************************************************************/

	/** (1) Texture/mesh to manage. Note that this becomes null when the texture/mesh is removed. */
	UStreamableRenderAsset*		RenderAsset;

	/** (1) SSAM handle for this asset. Consumed via FSimpleStreamableAssetManager::IsAssetRegistered
	 *  and GetRenderAssetScreenSize. Stale handles (asset unregistered) are silently ignored via
	 *  the table's generation check. */
	FSSAMAssetHandle SSAMAssetHandle;
	
	/** (1) The optional mip filename hash, see FRenderAssetStreamingManager::OnPakFileChanged() */
	FIoFilenameHash OptionalFileHash = INVALID_IO_FILENAME_HASH;
	/** (1) Cached texture/mesh LOD group. */
	int32			LODGroup;
	/** (1) Sum of all boost factors that applies to this texture/mesh. */
	float			BoostFactor;
	/** (1) Cached memory sizes for each possible mipcount. */
	union
	{
		uint32 CumulativeLODSizes[FStreamableRenderResourceState::MAX_LOD_COUNT];
		struct
		{
			uint32 CumulativeLODSizes_Mesh[FStreamableRenderResourceState::MAX_LOD_COUNT / 2];
			// Normalized size of projected bounding sphere - [0, 1]
			float LODScreenSizes[FStreamableRenderResourceState::MAX_LOD_COUNT / 2];
		};
	};
	// Make sure static mesh LODs will fit in the above struct
	static_assert(FStreamableRenderResourceState::MAX_LOD_COUNT / 2 >= MAX_STATIC_MESH_LODS);

	/** (2) How much game time has elapsed since the texture was bound for rendering. Based on FApp::GetCurrentTime(). */
	float			LastRenderTime;
	/** (3) FApp::GetCurrentTime() at the most recent instance-location removal (seed: -FLT_MAX in game, current time in editor). */
	FAtomicTimestamp	InstanceRemovedTimestamp;
	/** (3) Extra gameplay boost factor. Reset after every update. */
	float			DynamicBoostFactor;

	/** (4) Retention priority used to sacrifice mips when out of budget. */
	int32			RetentionPriority;
	/** (4) Normalized screen size. Only used by meshes. */
	float			NormalizedScreenSize;
	/** (4) The load request priority. */
	int32			LoadOrderPriority;	
	/** (4) A persistent bias applied to this texture. Increase whenever the streamer needs to make sacrifices to fit in budget */
	int32			BudgetMipBias;
	/** (2) LOD bias applied to no-ref meshes */
	int32			NoRefLODBias;
	/** (4) The mip that will be requested. Note, that some texture are loaded using split requests, so that the first request can be smaller than budgeted mip. */
	int8			WantedMips;
	/** (4) Number of LODs forced resident. Only used by meshes currently */
	int8			NumForcedMips;
	/** (2) Cached number of mip-maps in memory (including the base mip) */
	int8			ResidentMips;
	/** (2) Min number of mip-maps requested by the streaming system. */
	int8			RequestedMips;
	/** (2) Min mip to be requested by the streaming  */
	int8			MinAllowedMips;
	/** (2) Max mip to be requested by the streaming  */
	int8			MaxAllowedMips;
	/** (4) How many mips are missing to satisfy ideal quality because of max size limitation. Used to prevent sacrificing mips that are already visually sacrificed. */
	int8			NumMissingMips;
	/** (4) Max wanted mips for visible instances.*/
	int8			VisibleWantedMips;
	/** (4) Wanted mips for non visible instances.*/
	int8			HiddenWantedMips;
	/** (4) The max allowed mips (based on Visible and Hidden wanted mips) in order to fit in budget. */
	int8			BudgetedMips;

	/** (5) */
	int8 CachedWantedMips;
	/** (5) */
	int8 CachedVisibleWantedMips;

	/** (1) */
	EStreamableRenderAssetType RenderAssetType;
	/** (2) Cached state on disk of the optional mips for this streaming texture */
	EOptionalMipsState	OptionalMipsState;

	union
	{
		/** Atomic bitset holding all the per-asset boolean flags. See FStreamingRenderAssetFlags::EBit
		 *  for the bit assignments and category (1)-(5). Accessed only via GetFlag / SetFlag, which
		 *  issue single-bit atomic loads / fetch_or / fetch_and stores immune to the read-modify-write
		 *  byte-store races that the old ': 1' bitfield layout exposed when GT and async-task threads
		 *  touched bits in the same byte. */
		FStreamingRenderAssetFlags Flags;
		struct
		{
			UE_DEPRECATED(5.8, "Access these bits using SetFlag/GetFlag methods")
			uint8			bIsVirtualTexture : 1;
			UE_DEPRECATED(5.8, "Access these bits using SetFlag/GetFlag methods")
			uint8			bLoadWithHigherPriority : 1;
			UE_DEPRECATED(5.8, "Access these bits using SetFlag/GetFlag methods")
			uint8			bIsTerrainTexture : 1;
			UE_DEPRECATED(5.8, "Access these bits using SetFlag/GetFlag methods")
			uint8			bForceFullyLoad : 1;
			UE_DEPRECATED(5.8, "Access these bits using SetFlag/GetFlag methods")
			uint8			bIgnoreStreamingMipBias : 1;
			UE_DEPRECATED(5.8, "Access these bits using SetFlag/GetFlag methods")
			uint8			bHasUpdatePending : 1;
			UE_DEPRECATED(5.8, "Access these bits using SetFlag/GetFlag methods")
			uint8			bForceFullyLoadHeuristic : 1;
			UE_DEPRECATED(5.8, "Access these bits using SetFlag/GetFlag methods")
			uint8			bUseUnkownRefHeuristic : 1;
			UE_DEPRECATED(5.8, "Access these bits using SetFlag/GetFlag methods")
			uint8			bLooksLowRes : 1;
			UE_DEPRECATED(5.8, "Access these bits using SetFlag/GetFlag methods")
			uint8			bMissingTooManyMips : 1;
			UE_DEPRECATED(5.8, "Access these bits using SetFlag/GetFlag methods")
			uint8			bCachedForceFullyLoadHeuristic : 1;
		};
	};

	/** (5) */
	TBitArray<>		LevelIndexUsage;

private:
	FORCEINLINE_DEBUGGABLE void StreamWantedMips_Internal(FRenderAssetStreamingManager& Manager, bool bUseCachedData);

	FORCEINLINE_DEBUGGABLE int32 ClampMaxResChange_Internal(int32 NumMipDropRequested) const;
};
