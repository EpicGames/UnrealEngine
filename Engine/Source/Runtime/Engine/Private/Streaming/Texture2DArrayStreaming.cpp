// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
Texture2DArrayStreaming.cpp: Helpers to stream in and out texture 2D array LODs.
=============================================================================*/

#include "Streaming/Texture2DArrayStreaming.h"
#include "RHIContext.h"
#include "RenderUtils.h"
#include "Rendering/StreamableTextureResource.h"
#include "RenderingThread.h"
#include "Streaming/TextureMipAllocator.h"

//*****************************************************************************
//******************* FTexture2DArrayMipAllocator_Reallocate ******************
//*****************************************************************************

FTexture2DArrayMipAllocator_Reallocate::FTexture2DArrayMipAllocator_Reallocate(UTexture* Texture)
	: FTextureMipAllocator(Texture, ETickState::AllocateMips, ETickThread::Async)
{
}

FTexture2DArrayMipAllocator_Reallocate::~FTexture2DArrayMipAllocator_Reallocate()
{
}

bool FTexture2DArrayMipAllocator_Reallocate::AllocateMips(const FTextureUpdateContext& Context, FTextureMipInfoArray& OutMipInfos, const FTextureUpdateSyncOptions& SyncOptions)
{
	OutMipInfos.AddDefaulted(CurrentFirstLODIdx);
	StreamedMipData.AddDefaulted(CurrentFirstLODIdx);
	StreamedSliceSize.AddZeroed(CurrentFirstLODIdx);

	// Allocate the mip memory as temporary buffers so that the FTextureMipDataProvider implementation can write to it.
	for (int32 MipIdx = PendingFirstLODIdx; MipIdx < CurrentFirstLODIdx; ++MipIdx)
	{
		const FTexture2DMipMap& OwnerMip = *Context.MipsView[MipIdx];
		FTextureMipInfo& MipInfo = OutMipInfos[MipIdx];

		// Note that streamed in mips size will always be at least as big as block size.
		MipInfo.Format = Context.Resource->GetPixelFormat();
		MipInfo.SizeX = OwnerMip.SizeX;
		MipInfo.SizeY = OwnerMip.SizeY;
		MipInfo.ArraySize = OwnerMip.SizeZ;

		uint32 TextureAlign = 0;

		MipInfo.DataSize = MipInfo.ArraySize * CalcTextureMipMapSize(MipInfo.SizeX, MipInfo.SizeY, MipInfo.Format, 0);
		MipInfo.DestData = FMemory::Malloc(MipInfo.DataSize, 16);

		check(MipInfo.DataSize % MipInfo.ArraySize == 0);

		StreamedMipData[MipIdx].Reset((uint8*)MipInfo.DestData);
		StreamedSliceSize[MipIdx] = MipInfo.DataSize / MipInfo.ArraySize;
	}

	AdvanceTo(ETickState::UploadMips, ETickThread::Render);
	return true;
}

bool FTexture2DArrayMipAllocator_Reallocate::UploadMips(const FTextureUpdateContext& Context, const FTextureUpdateSyncOptions& SyncOptions)
{
	FRHICommandListImmediate& RHICmdList = FRHICommandListImmediate::Get();

	const uint32 NumSlices = Context.Resource->GetSizeZ();

	// Create new Texture.
	{
		const FTexture2DMipMap& FirstMip = *Context.MipsView[PendingFirstLODIdx];

		const FRHITextureCreateDesc Desc =
			FRHITextureCreateDesc::Create2DArray(TEXT("FinalizeMips"), FirstMip.SizeX, FirstMip.SizeY, FirstMip.SizeZ, Context.Resource->GetPixelFormat())
			.SetNumMips(ResourceState.NumRequestedLODs)
			.SetFlags(Context.Resource->GetCreationFlags())
			.SetExtData(Context.Resource->GetExtData());

		IntermediateTextureRHI = RHICmdList.CreateTexture(Desc);
	}

	// Copy shared mips.
	{
		FRHITexture* SrcTexture = Context.Resource->GetTexture2DArrayRHI();
		FRHITexture* DstTexture = IntermediateTextureRHI.GetReference();

		UE::RHI::CopySharedMips_AssumeSRVMaskState(RHICmdList, SrcTexture, DstTexture);
	}

	// Update the streamed in mips if they were not initialized from the bulk data.
	for (int32 MipIdx = PendingFirstLODIdx; MipIdx < CurrentFirstLODIdx; ++MipIdx)
	{
		const FTexture2DMipMap& Mip = *Context.MipsView[MipIdx];
		const uint8* MipData = StreamedMipData[MipIdx].Get();
		const uint64 SlizeSize = StreamedSliceSize[MipIdx];
		if (MipData && SlizeSize)
		{
			const int32 RHIMipIndex = MipIdx - PendingFirstLODIdx;
			for (uint32 SliceIdx = 0; SliceIdx < NumSlices; ++SliceIdx)
			{
				uint32 DestStride = 0;
				void* DestData = RHICmdList.LockTexture2DArray(IntermediateTextureRHI, SliceIdx, RHIMipIndex, RLM_WriteOnly, DestStride, false);
				FMemory::Memcpy(DestData, MipData + SliceIdx * SlizeSize, SlizeSize);
				RHICmdList.UnlockTexture2DArray(IntermediateTextureRHI, SliceIdx, RHIMipIndex, false);
			}

		}
	}
	StreamedMipData.Empty();
	StreamedSliceSize.Empty();

	AdvanceTo(ETickState::Done, ETickThread::None);
	return true;
}

void FTexture2DArrayMipAllocator_Reallocate::Cancel(const FTextureUpdateSyncOptions& SyncOptions)
{
	// Release the temporary mip data. Can be run on either renderthread or async threads.
	StreamedMipData.Empty();
	StreamedSliceSize.Empty();
}

FTextureMipAllocator::ETickThread FTexture2DArrayMipAllocator_Reallocate::GetCancelThread() const
{
	// If there are only temporary mip data, they can be freed on any threads.
	if (StreamedMipData.Num())
	{
		return ETickThread::Async;
	}
	// Nothing to do.
	else
	{
		return ETickThread::None;
	}
}
