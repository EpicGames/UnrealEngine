// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
Texture2DArrayStreaming.h: Helpers to stream in and out texture 2D array LODs.
=============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "RenderAssetUpdate.h"
#include "Engine/Texture2DArray.h"
#include "Rendering/Texture2DResource.h"
#include "Texture2DMipDataProvider_DDC.h"
#include "Texture2DMipDataProvider_IO.h"
#include "TextureMipAllocator.h"

#if WITH_EDITORONLY_DATA
typedef FTexture2DMipDataProvider_DDC FTexture2DArrayMipDataProvider_DDC;
#endif

typedef FTexture2DMipDataProvider_IO FTexture2DArrayMipDataProvider_IO;

/**
* Implements a texture3d mip resizing strategy based on creating a duplicate texture and copyinng the shared mips.
*/
class FTexture2DArrayMipAllocator_Reallocate : public FTextureMipAllocator
{
public:

	FTexture2DArrayMipAllocator_Reallocate(UTexture* Texture);
	~FTexture2DArrayMipAllocator_Reallocate();

	// ********************************************************
	// ********* FTextureMipAllocator implementation **********
	// ********************************************************

	bool AllocateMips(const FTextureUpdateContext& Context, FTextureMipInfoArray& OutMipInfos, const FTextureUpdateSyncOptions& SyncOptions) final override;
	bool UploadMips(const FTextureUpdateContext& Context, const FTextureUpdateSyncOptions& SyncOptions) final override;
	void Cancel(const FTextureUpdateSyncOptions& SyncOptions) final override;
	ETickThread GetCancelThread() const final override;

protected:

	// Used to free memory allocated from a TUniquePtr
	struct FMallocDeleter
	{
		void operator() (uint8* Buffer)
		{
			FMemory::Free(Buffer);
		}
	};

	// The temporary main memory allocations holding the mip data
	TArray<TUniquePtr<uint8, FMallocDeleter>, TInlineAllocator<MAX_TEXTURE_MIP_COUNT>> StreamedMipData;
	TArray<uint64, TInlineAllocator<MAX_TEXTURE_MIP_COUNT>> StreamedSliceSize;
};
