// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
Texture2DStreamIn_AsyncCreate.cpp: Implementation of FTextureMipAllocator using RHIAsyncCreateTexture2D
=============================================================================*/

#include "Texture2DMipAllocator_AsyncCreate.h"
#include "RenderUtils.h"
#include "Rendering/StreamableTextureResource.h"
#include "Streaming/TextureMipAllocator.h"

FTexture2DMipAllocator_AsyncCreate::FTexture2DMipAllocator_AsyncCreate(UTexture* Texture)
	: FTextureMipAllocator(Texture, ETickState::AllocateMips, ETickThread::Async)
{
}

FTexture2DMipAllocator_AsyncCreate::~FTexture2DMipAllocator_AsyncCreate()
{
	check(!FinalMipData.Num());
}

bool FTexture2DMipAllocator_AsyncCreate::AllocateMips(
	const FTextureUpdateContext& Context, 
	FTextureMipInfoArray& OutMipInfos, 
	const FTextureUpdateSyncOptions& SyncOptions)
{
	check(PendingFirstLODIdx < CurrentFirstLODIdx);

	if (!Context.Resource)
	{
		return false;
	}

	OutMipInfos.AddDefaulted(CurrentFirstLODIdx);

	// Allocate the mip memory as temporary buffers so that the FTextureMipDataProvider implementation can write to it.
	for (int32 MipIndex = PendingFirstLODIdx; MipIndex < CurrentFirstLODIdx; ++MipIndex)
	{
		const FTexture2DMipMap& OwnerMip = *Context.MipsView[MipIndex];
		FTextureMipInfo& MipInfo = OutMipInfos[MipIndex];

		MipInfo.Format = Context.Resource->GetPixelFormat();
		MipInfo.SizeX = OwnerMip.SizeX;
		MipInfo.SizeY = OwnerMip.SizeY;
		MipInfo.DataSize = CalcTextureMipMapSize(MipInfo.SizeX, MipInfo.SizeY, MipInfo.Format, 0);
		// Allocate the mip in main memory. It will later be used to create the mips with proper initial states (without going through lock/unlock).
		MipInfo.DestData = FMemory::Malloc(MipInfo.DataSize);

		// Backup the allocated memory so that it can safely be freed.
		FinalMipData.Add(MipInfo.DestData);
	}

	// Backup size and format.
	if (OutMipInfos.IsValidIndex(PendingFirstLODIdx))
	{
		FinalSizeX = OutMipInfos[PendingFirstLODIdx].SizeX;
		FinalSizeY = OutMipInfos[PendingFirstLODIdx].SizeY;
		FinalFormat = OutMipInfos[PendingFirstLODIdx].Format;

		// Once the FTextureMipDataProvider has set the mip data, UploadMips can then create the texture.
		AdvanceTo(ETickState::UploadMips, ETickThread::Async);
		return true;
	}
	else // No new mips? something is wrong.
	{
		return false;
	}
}

// This gets called 2 times :
// - Async : create the texture with the mips data
// - Render : swap the results
bool FTexture2DMipAllocator_AsyncCreate::UploadMips(const FTextureUpdateContext& Context, const FTextureUpdateSyncOptions& SyncOptions)
{
	FRHITexture* Texture2DRHI = Context.Resource ? Context.Resource->GetTexture2DRHI() : nullptr;
	if (!Texture2DRHI)
	{
		return false;
	}

	// Step (1) : Create the texture on the async thread, having the new mip data as reference so that it can be initialized correctly.
	if (!IntermediateTextureRHI)
	{
		// Create the intermediate texture.
		FGraphEventRef CompletionEvent;
		IntermediateTextureRHI = RHIAsyncCreateTexture2D(
			FinalSizeX,
			FinalSizeY,
			FinalFormat,
			ResourceState.NumRequestedLODs,
			Context.Resource->GetCreationFlags(),
			ERHIAccess::Unknown,
			FinalMipData.GetData(),
			FinalMipData.Num(),
			*Context.Resource->GetTextureName().ToString(),
			CompletionEvent);

		if (CompletionEvent)
		{
			SyncOptions.Counter->Increment();
			FFunctionGraphTask::CreateAndDispatchWhenReady(
				[SyncCounter = SyncOptions.Counter]()
				{
					SyncCounter->Decrement();
				},
				TStatId{},
				CompletionEvent);
		}

		// Free the temporary mip data, since copy is now in the RHIAsyncCreateTexture2D command.
		ReleaseAllocatedMipData();

		// Go to next step, on the renderthread.
		AdvanceTo(ETickState::UploadMips, ETickThread::Render);
	}
	// Step (2) : Copy the non initialized mips on the using RHICopySharedMips, must run on the renderthread.
	else
	{
		// Copy the mips.
		UE::RHI::CopySharedMips_AssumeSRVMaskState(
			FRHICommandListExecutor::GetImmediateCommandList(),
			Texture2DRHI,
			IntermediateTextureRHI);
		// Go to next step, on the renderthread.
		AdvanceTo(ETickState::Done, ETickThread::None);
	}
	return true;
}

void FTexture2DMipAllocator_AsyncCreate::Cancel(const FTextureUpdateSyncOptions& SyncOptions)
{
	// Release the temporary mip data. Can be run on either renderthread or async threads.
	ReleaseAllocatedMipData();
}

FTextureMipAllocator::ETickThread FTexture2DMipAllocator_AsyncCreate::GetCancelThread() const
{
	// If there are only temporary mip data, they can be freed on any threads.
	if (FinalMipData.Num())
	{
		return ETickThread::Async;
	}
	// Nothing to do.
	else
	{
		return ETickThread::None;
	}
}

void FTexture2DMipAllocator_AsyncCreate::ReleaseAllocatedMipData()
{
	// Release the temporary mip data.
	for (void* NewData : FinalMipData)
	{
		if (NewData)
		{
			FMemory::Free(NewData);
		}
	}
	FinalMipData.Empty();
}
