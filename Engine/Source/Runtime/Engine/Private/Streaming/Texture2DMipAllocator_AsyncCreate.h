// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
Texture2DStreamIn_AsyncCreate.h: Implementation of FTextureMipAllocator using RHIAsyncCreateTexture2D
=============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "TextureMipAllocator.h"
#include "Engine/Texture2D.h"
#include "Rendering/Texture2DResource.h"

class UTexture2D;

/**
* FTexture2DMipAllocator_AsyncCreate implements FTextureMipAllocator using RHIAsyncCreateTexture2D.
* It uses a more efficient path than FTexture2DMipAllocator_AsyncReallocate because it can run some
* steps on async threads (and not just on the render thread). 
* Can only be used when GRHISupportsAsyncTextureCreation is true.
*/
class FTexture2DMipAllocator_AsyncCreate : public FTextureMipAllocator
{
public:

	FTexture2DMipAllocator_AsyncCreate(UTexture* Texture);
	~FTexture2DMipAllocator_AsyncCreate();

	// ********************************************************
	// ********* FTextureMipAllocator implementation **********
	// ********************************************************

	bool AllocateMips(const FTextureUpdateContext& Context, FTextureMipInfoArray& OutMipInfos, const FTextureUpdateSyncOptions& SyncOptions) final override;
	bool UploadMips(const FTextureUpdateContext& Context, const FTextureUpdateSyncOptions& SyncOptions) final override;
	void Cancel(const FTextureUpdateSyncOptions& SyncOptions) final override;
	ETickThread GetCancelThread() const final override;

protected:

	// Release the temporary buffers referenced in FinalMipData.
	void ReleaseAllocatedMipData();

	// The final resolution x and y of the texture 2d.
	int32 FinalSizeX = 0;
	int32 FinalSizeY = 0;
	// The initial and final format of the texture.
	EPixelFormat FinalFormat = EPixelFormat::PF_Unknown;
	// The temporary main memory allocations holding the mip data
	TArray<void*, TInlineAllocator<MAX_TEXTURE_MIP_COUNT>> FinalMipData;
};

