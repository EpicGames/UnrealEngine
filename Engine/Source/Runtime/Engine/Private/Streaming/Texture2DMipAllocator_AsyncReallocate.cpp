// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
Texture2DStreamIn_AsyncReallocate.cpp: Load texture 2D mips using ITextureMipDataProvider
=============================================================================*/

#include "Texture2DMipAllocator_AsyncReallocate.h"
#include "RenderUtils.h"
#include "Rendering/StreamableTextureResource.h"
#include "Streaming/TextureMipAllocator.h"

extern TAutoConsoleVariable<int32> CVarFlushRHIThreadOnSTreamingTextureLocks;

FTexture2DMipAllocator_AsyncReallocate::FTexture2DMipAllocator_AsyncReallocate(UTexture* Texture)
	: FTextureMipAllocator(Texture, ETickState::AllocateMips, ETickThread::Render)
{
}

FTexture2DMipAllocator_AsyncReallocate::~FTexture2DMipAllocator_AsyncReallocate()
{
	check(!LockedMipIndices.Num());
}

// ********************************************************
// ********* FTextureMipAllocator implementation **********
// ********************************************************

bool FTexture2DMipAllocator_AsyncReallocate::AllocateMips(
	const FTextureUpdateContext& Context, 
	FTextureMipInfoArray& OutMipInfos, 
	const FTextureUpdateSyncOptions& SyncOptions)
{
	check(PendingFirstLODIdx < CurrentFirstLODIdx);

	FRHITexture* Texture2DRHI = Context.Resource ? Context.Resource->GetTexture2DRHI() : nullptr;
	if (!Texture2DRHI)
	{
		return false;
	}

	FRHICommandListImmediate& RHICmdList = FRHICommandListImmediate::Get();

	// Step (1) : Create the texture on the render thread using AsyncReallocateTexture2D. Wait for the RHI to signal the operation is completed through the FThreadSafeCounter.
	if (!IntermediateTextureRHI)
	{
		const FTexture2DMipMap& OwnerMip = *Context.MipsView[PendingFirstLODIdx];
		check(SyncOptions.Counter);

		SyncOptions.Counter->Increment();

		IntermediateTextureRHI = RHICmdList.AsyncReallocateTexture2D(
			Texture2DRHI,
			ResourceState.NumRequestedLODs,
			OwnerMip.SizeX,
			OwnerMip.SizeY,
			SyncOptions.Counter);

		// Run the next step, when IntermediateTextureRHI will be somewhat valid (after synchronization).
		AdvanceTo(ETickState::AllocateMips, ETickThread::Render);
		return true;
	}
	// Step (2) : Finalize the texture state through FinalizeAsyncReallocateTexture2D and lock the new mips.
	else
	{
		const bool bFlushRHIThread = CVarFlushRHIThreadOnSTreamingTextureLocks.GetValueOnAnyThread() > 0;

		OutMipInfos.AddDefaulted(CurrentFirstLODIdx);

		for (int32 MipIndex = PendingFirstLODIdx; MipIndex < CurrentFirstLODIdx; ++MipIndex)
		{
			const FTexture2DMipMap& OwnerMip = *Context.MipsView[MipIndex];
			FTextureMipInfo& MipInfo = OutMipInfos[MipIndex];

			MipInfo.Format = Context.Resource->GetPixelFormat();
			MipInfo.SizeX = OwnerMip.SizeX;
			MipInfo.SizeY = OwnerMip.SizeY;
#if WITH_EDITORONLY_DATA
			MipInfo.DataSize = CalcTextureMipMapSize(MipInfo.SizeX, MipInfo.SizeY, MipInfo.Format, 0);
#else // Hasn't really been used on console. To investigate!
			MipInfo.DataSize = 0;
#endif
			MipInfo.DestData = RHICmdList.LockTexture2D(IntermediateTextureRHI, MipIndex - PendingFirstLODIdx, RLM_WriteOnly, MipInfo.RowPitch, false, bFlushRHIThread);

			// Add this mip in the locked list of mips so that it can safely be unlocked when needed.
			LockedMipIndices.Add(MipIndex - PendingFirstLODIdx);
		}

		// New mips are ready to be unlocked by the FTextureMipDataProvider implementation.
		AdvanceTo(ETickState::UploadMips, ETickThread::Render);
		return true;
	}
}

bool FTexture2DMipAllocator_AsyncReallocate::UploadMips(const FTextureUpdateContext& Context, const FTextureUpdateSyncOptions& SyncOptions)
{
	if (!IntermediateTextureRHI)
	{
		return false;
	}

	// Unlock the mips so that the texture can be updated.
	UnlockNewMips();

	AdvanceTo(ETickState::Done, ETickThread::None);
	return true;
}

void FTexture2DMipAllocator_AsyncReallocate::Cancel(const FTextureUpdateSyncOptions& SyncOptions)
{
	// Unlock any locked mips.
	UnlockNewMips();
}

FTextureMipAllocator::ETickThread FTexture2DMipAllocator_AsyncReallocate::GetCancelThread() const
{
	// Nothing to do.
	return ETickThread::None;
}

// ****************************
// ********* Helpers **********
// ****************************

void FTexture2DMipAllocator_AsyncReallocate::UnlockNewMips()
{
	// Unlock any locked mips.
	if (IntermediateTextureRHI)
	{
		FRHICommandListImmediate& RHICmdList = FRHICommandListImmediate::Get();

		const bool bFlushRHIThread = CVarFlushRHIThreadOnSTreamingTextureLocks.GetValueOnAnyThread() > 0;
		for (int32 MipIndex : LockedMipIndices)
		{
			RHICmdList.UnlockTexture2D(IntermediateTextureRHI, MipIndex, false, bFlushRHIThread);
		}
		LockedMipIndices.Empty();
	}
}
