// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "TextureMipAllocator.h"
#include "Engine/Texture2D.h"
#include "Rendering/Texture2DResource.h"

class UTexture2D;

/**
* FTexture2DMipAllocator_AsyncReallocate is the default implementation of  FTextureMipAllocator.
* It uses AsyncReallocateTexture2D which creates a new texture on the render thread and copies the mips if required.
*/
class FTexture2DMipAllocator_AsyncReallocate : public FTextureMipAllocator
{
public:

	FTexture2DMipAllocator_AsyncReallocate(UTexture* Texture);
	~FTexture2DMipAllocator_AsyncReallocate();

	// ********************************************************
	// ********* FTextureMipAllocator implementation **********
	// ********************************************************

	bool AllocateMips(const FTextureUpdateContext& Context, FTextureMipInfoArray& OutMipInfos, const FTextureUpdateSyncOptions& SyncOptions) final override;
	bool UploadMips(const FTextureUpdateContext& Context, const FTextureUpdateSyncOptions& SyncOptions) final override;
	void Cancel(const FTextureUpdateSyncOptions& SyncOptions) final override;
	ETickThread GetCancelThread() const final override;

protected:

	// Unlock the mips referenced in LockedMipIndices.
	void UnlockNewMips();

	// The list of mips that are currently locked.
	TArray<int32, TInlineAllocator<MAX_TEXTURE_MIP_COUNT>> LockedMipIndices;
};
