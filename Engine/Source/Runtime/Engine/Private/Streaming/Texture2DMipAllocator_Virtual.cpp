// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
Texture2DStreamIn_Virtual.cpp: Load texture 2D mips using ITextureMipDataProvider
=============================================================================*/

#include "Texture2DMipAllocator_Virtual.h"
#include "RenderUtils.h"
#include "Rendering/StreamableTextureResource.h"
#include "Streaming/TextureMipAllocator.h"
#include "Containers/ResourceArray.h"

extern TAutoConsoleVariable<int32> CVarFlushRHIThreadOnSTreamingTextureLocks;

FTexture2DMipAllocator_Virtual::FTexture2DMipAllocator_Virtual(UTexture* Texture)
	: FTextureMipAllocator(Texture, ETickState::AllocateMips, ETickThread::Render)
{
}

FTexture2DMipAllocator_Virtual::~FTexture2DMipAllocator_Virtual()
{
	check(!LockedMipIndices.Num());
}

// ********************************************************
// ********* FTextureMipAllocator implementation **********
// ********************************************************

bool FTexture2DMipAllocator_Virtual::AllocateMips(
	const FTextureUpdateContext& Context, 
	FTextureMipInfoArray& OutMipInfos, 
	const FTextureUpdateSyncOptions& SyncOptions)
{
	check(PendingFirstLODIdx < CurrentFirstLODIdx);

	FRHITexture* Texture2DRHI = Context.Resource ? Context.Resource->GetTexture2DRHI() : nullptr;
	if (!Texture2DRHI)
	{
		return false;
	}

	FRHICommandListImmediate& RHICmdList = FRHICommandListImmediate::Get();

	// Step (1) : Create the texture on the renderthread .......
	if (!IntermediateTextureRHI)
	{
		DoConvertToVirtualWithNewMips(RHICmdList, Context);
		// Run the next step, when IntermediateTextureRHI will be somewhat valid (after synchronization).
		AdvanceTo(ETickState::AllocateMips, ETickThread::Render);
		return true;
	}
	// Step (2) : Finalize the texture state through.... .
	else
	{
		const bool bFlushRHIThread = CVarFlushRHIThreadOnSTreamingTextureLocks.GetValueOnAnyThread() > 0;

		RHICmdList.VirtualTextureSetFirstMipInMemory(IntermediateTextureRHI, PendingFirstLODIdx);

		OutMipInfos.AddDefaulted(CurrentFirstLODIdx);

		for (int32 MipIndex = PendingFirstLODIdx; MipIndex < CurrentFirstLODIdx; ++MipIndex)
		{
			const FTexture2DMipMap& OwnerMip = *Context.MipsView[MipIndex];
			FTextureMipInfo& MipInfo = OutMipInfos[MipIndex];

			MipInfo.Format = Context.Resource->GetPixelFormat();
			MipInfo.SizeX = OwnerMip.SizeX;
			MipInfo.SizeY = OwnerMip.SizeY;
#if WITH_EDITORONLY_DATA
			MipInfo.DataSize = CalcTextureMipMapSize(MipInfo.SizeX, MipInfo.SizeY, MipInfo.Format, 0);
#else // Hasn't really been used on console. To investigate!
			MipInfo.DataSize = 0;
#endif

			MipInfo.DestData = RHICmdList.LockTexture2D(IntermediateTextureRHI, MipIndex, RLM_WriteOnly, MipInfo.RowPitch, false, bFlushRHIThread);

			// Add this mip in the locked list of mips so that it can safely be unlocked when needed.
			LockedMipIndices.Add(MipIndex);
		}

		// New mips are ready to be unlocked by the FTextureMipDataProvider implementation.
		AdvanceTo(ETickState::UploadMips, ETickThread::Render);
		return true;
	}
}

bool FTexture2DMipAllocator_Virtual::UploadMips(const FTextureUpdateContext& Context, const FTextureUpdateSyncOptions& SyncOptions)
{
	if (!IntermediateTextureRHI)
	{
		return false;
	}

	FRHICommandListImmediate& RHICmdList = FRHICommandListImmediate::Get();

	// Unlock the mips so that the texture can be updated.
	UnlockNewMips(RHICmdList);

	if(IntermediateTextureRHI)
	{
		RHICmdList.VirtualTextureSetFirstMipVisible(IntermediateTextureRHI, PendingFirstLODIdx);
	}

	AdvanceTo(ETickState::Done, ETickThread::None);
	return true;
}

void FTexture2DMipAllocator_Virtual::Cancel(const FTextureUpdateSyncOptions& SyncOptions)
{
	if (IntermediateTextureRHI)
	{
		FRHICommandListImmediate& RHICmdList = FRHICommandListImmediate::Get();

		// Unlock any locked mips.
		UnlockNewMips(RHICmdList);

		RHICmdList.VirtualTextureSetFirstMipInMemory(IntermediateTextureRHI, CurrentFirstLODIdx);
	}
}

FTextureMipAllocator::ETickThread FTexture2DMipAllocator_Virtual::GetCancelThread() const
{
	// Nothing to do.
	return ETickThread::None;
}

// ****************************
// ********* Helpers **********
// ****************************

void FTexture2DMipAllocator_Virtual::UnlockNewMips(FRHICommandListImmediate& RHICmdList)
{
	// Unlock any locked mips.
	if (IntermediateTextureRHI)
	{
		const bool bFlushRHIThread = CVarFlushRHIThreadOnSTreamingTextureLocks.GetValueOnAnyThread() > 0;
		for (int32 MipIndex : LockedMipIndices)
		{
			RHICmdList.UnlockTexture2D(IntermediateTextureRHI, MipIndex, false, CVarFlushRHIThreadOnSTreamingTextureLocks.GetValueOnAnyThread() > 0 );
		}
		LockedMipIndices.Empty();
	}
}

void FTexture2DMipAllocator_Virtual::DoConvertToVirtualWithNewMips(FRHICommandListImmediate& RHICmdList, const FTextureUpdateContext& Context)
{
	if (Context.Resource)
	{
		// If the texture is not virtual, then make it virtual immediately.
		if (!Context.Resource->GetTexture2DRHI() || (Context.Resource->GetTexture2DRHI()->GetFlags() & ETextureCreateFlags::Virtual) != ETextureCreateFlags::Virtual)
		{
			const FTexture2DMipMap& MipMap0 = *Context.MipsView[0];

			FTexture2DResource* Resource2D = Context.Resource->GetTexture2DResource();

			// Create a copy of the texture that is a virtual texture.
			FRHITextureCreateDesc CreateDesc =
				FRHITextureCreateDesc::Create2D(TEXT("FTexture2DUpdate"), MipMap0.SizeX, MipMap0.SizeY, Context.Resource->GetPixelFormat())
				.SetNumMips(ResourceState.MaxNumLODs)
				.SetFlags(Context.Resource->GetCreationFlags() | ETextureCreateFlags::Virtual);
			if (Resource2D->ResourceMem)
			{
				CreateDesc.SetInitActionBulkData(Resource2D->ResourceMem);
			}

			IntermediateTextureRHI = RHICmdList.CreateTexture(CreateDesc);

			RHICmdList.VirtualTextureSetFirstMipInMemory(IntermediateTextureRHI, CurrentFirstLODIdx);
			RHICmdList.VirtualTextureSetFirstMipVisible(IntermediateTextureRHI, CurrentFirstLODIdx);

			UE::RHI::CopySharedMips_AssumeSRVMaskState(
				RHICmdList,
				Context.Resource->GetTexture2DRHI(),
				IntermediateTextureRHI);
		}
		else
		{
			// Otherwise the current texture is already virtual and we can update it directly.
			IntermediateTextureRHI = Context.Resource->GetTexture2DRHI();
		}
	}
}
