// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "TextureMipAllocator.h"
#include "Engine/Texture2D.h"
#include "Rendering/Texture2DResource.h"

class UTexture2D;

/**
* FTexture2DMipAllocator_Virtual is an allocator specialized in creating and managing virtual textures.
*/
class FTexture2DMipAllocator_Virtual : public FTextureMipAllocator
{
public:

	FTexture2DMipAllocator_Virtual(UTexture* Texture);
	~FTexture2DMipAllocator_Virtual();

	// ********************************************************
	// ********* FTextureMipAllocator implementation **********
	// ********************************************************

	bool AllocateMips(const FTextureUpdateContext& Context, FTextureMipInfoArray& OutMipInfos, const FTextureUpdateSyncOptions& SyncOptions) final override;
	bool UploadMips(const FTextureUpdateContext& Context, const FTextureUpdateSyncOptions& SyncOptions) final override;
	void Cancel(const FTextureUpdateSyncOptions& SyncOptions) final override;
	ETickThread GetCancelThread() const final override;

protected:

	// Unlock the mips referenced in LockedMipIndices.
	void UnlockNewMips(FRHICommandListImmediate& RHICmdList);

	void DoConvertToVirtualWithNewMips(FRHICommandListImmediate& RHICmdList, const FTextureUpdateContext& Context);

	// The list of mips that are currently locked.
	TArray<int32, TInlineAllocator<MAX_TEXTURE_MIP_COUNT>> LockedMipIndices;
};
