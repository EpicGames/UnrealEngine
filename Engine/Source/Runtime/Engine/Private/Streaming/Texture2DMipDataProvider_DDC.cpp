// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
Texture2DMipDataProvider_DDC.cpp : Implementation of FTextureMipDataProvider using DDC requests.
=============================================================================*/

#include "Texture2DMipDataProvider_DDC.h"
#include "EngineLogs.h"
#include "Rendering/StreamableTextureResource.h"
#include "Serialization/MemoryReader.h"
#include "TextureCompiler.h"

#if WITH_EDITORONLY_DATA

#include "DerivedDataCache.h"

FTexture2DMipDataProvider_DDC::FTexture2DMipDataProvider_DDC(UTexture* InTexture)
	: FTextureMipDataProvider(InTexture, ETickState::Init, ETickThread::Async)
	, DDCRequestOwner(UE::DerivedData::EPriority::Normal)
	, Texture(InTexture)
{
}

FTexture2DMipDataProvider_DDC::~FTexture2DMipDataProvider_DDC()
{
}

void FTexture2DMipDataProvider_DDC::Init(const FTextureUpdateContext& Context, const FTextureUpdateSyncOptions& SyncOptions)
{
	if (DDCBuffers.IsEmpty())
	{
		DDCBuffers.AddZeroed(CurrentFirstLODIdx);

		FTexturePlatformData*const * PtrPlatformData = const_cast<UTexture*>(Context.Texture)->GetRunningPlatformData();
		if (PtrPlatformData && *PtrPlatformData)
		{
			const FTexturePlatformData* PlatformData = *PtrPlatformData;
			const int32 LODBias = static_cast<int32>(Context.MipsView.GetData() - PlatformData->Mips.GetData());

			using namespace UE::DerivedData;
			TArray<FCacheGetChunkRequest> MipKeys;

			TStringBuilder<256> MipNameBuilder;
			Context.Texture->GetPathName(nullptr, MipNameBuilder);
			const int32 TextureNameLen = MipNameBuilder.Len();

			if (PlatformData->DerivedDataKey.IsType<FString>())
			{
				for (int32 MipIndex = PendingFirstLODIdx; MipIndex < CurrentFirstLODIdx; ++MipIndex)
				{
					const FTexture2DMipMap& OwnerMip = *Context.MipsView[MipIndex];
					if (OwnerMip.IsPagedToDerivedData())
					{
						FCacheGetChunkRequest& Request = MipKeys.AddDefaulted_GetRef();
						MipNameBuilder.Appendf(TEXT(" [MIP %d]"), MipIndex + LODBias);
						Request.Name = MipNameBuilder;
						Request.Key = ConvertLegacyCacheKey(PlatformData->GetDerivedDataMipKeyString(MipIndex + LODBias, OwnerMip));
						Request.UserData = MipIndex;
						MipNameBuilder.RemoveSuffix(MipNameBuilder.Len() - TextureNameLen);
					}
				}
			}
			else if (PlatformData->DerivedDataKey.IsType<UE::DerivedData::FCacheKeyProxy>())
			{
				const FCacheKey& Key = *PlatformData->DerivedDataKey.Get<UE::DerivedData::FCacheKeyProxy>().AsCacheKey();
				for (int32 MipIndex = PendingFirstLODIdx; MipIndex < CurrentFirstLODIdx; ++MipIndex)
				{
					const FTexture2DMipMap& MipMap = *Context.MipsView[MipIndex];
					if (MipMap.IsPagedToDerivedData())
					{
						FCacheGetChunkRequest& Request = MipKeys.AddDefaulted_GetRef();
						MipNameBuilder.Appendf(TEXT(" [MIP %d]"), MipIndex + LODBias);
						Request.Name = MipNameBuilder;
						Request.Key = Key;
						Request.Id = FTexturePlatformData::MakeMipId(MipIndex + LODBias);
						Request.UserData = MipIndex;
						MipNameBuilder.RemoveSuffix(MipNameBuilder.Len() - TextureNameLen);
					}
				}
			}
			else
			{
				UE_LOGF(LogTexture, Error, "Attempting to stream data in an unsupported cache format for mips [%d, %d) of %ls.",
					PendingFirstLODIdx, CurrentFirstLODIdx, *Context.Texture->GetPathName());
			}

			if (MipKeys.Num())
			{
				FRequestBarrier Barrier(DDCRequestOwner);
				GetCache().GetChunks(MipKeys, DDCRequestOwner, [this](FCacheGetChunkResponse&& Response)
				{
					if (Response.Status == EStatus::Ok)
					{
						const int32 MipIndex = int32(Response.UserData);
						check(!DDCBuffers[MipIndex]);
						DDCBuffers[MipIndex] = MoveTemp(Response.RawData);
					}
					else if (Response.Status == EStatus::Error)
					{
						FTextureCompilingManager::Get().ForceDeferredTextureRebuildAnyThread({Texture});
					}
				});
			}
		}
		else
		{
			UE_LOGF(LogTexture, Error, "Attempting to stream data that has not been generated yet for mips [%d, %d) of %ls.",
				PendingFirstLODIdx, CurrentFirstLODIdx, *Context.Texture->GetPathName());
		}

		*SyncOptions.bSnooze = true;
	}
	else // The cache request has been issued, only check whether they are ready.
	{
		if (!DDCRequestOwner.Poll())
		{
			*SyncOptions.bSnooze = true;
			return;
		}

		AdvanceTo(ETickState::GetMips, ETickThread::Async);
	}
}

bool FTexture2DMipDataProvider_DDC::SerializeMipInfo(const FTextureUpdateContext& Context, FArchive& Ar, int32 MipIndex, int64 MipSize, const FTextureMipInfo& OutMipInfo)
{
	const uint32 DepthOrArraySize = FMath::Max<uint32>(OutMipInfo.ArraySize, OutMipInfo.SizeZ);
	if (MipSize == OutMipInfo.DataSize)
	{
		Ar.Serialize(OutMipInfo.DestData, MipSize);
		return true;
	}
	// This used to be DepthOrArraySize > 1, which fails when a volume texture gets mipped down to WxHx1 that also has
	// allocation alignment such that the size is larger than its bulk data size. The original author is gone, but this looks
	// like it should just be a 0 check to prevent divide by zero with erroneous data. (test case was 2048x2048x8 DXT1 on dx12)
	else if (uint64(MipSize) < OutMipInfo.DataSize && DepthOrArraySize > 0 && OutMipInfo.DataSize % DepthOrArraySize == 0 && MipSize % DepthOrArraySize == 0)
	{
		UE_LOG(LogTexture, Verbose, TEXT("Cached mip size is smaller than streaming buffer size for mip %d of %s: %" UINT64_FMT " KiB / %" INT64_FMT " KiB."),
			ResourceState.MaxNumLODs - MipIndex, *Context.Resource->GetTextureName().ToString(), OutMipInfo.DataSize / 1024, MipSize / 1024);

		const uint64 SourceSubSize = MipSize / DepthOrArraySize;
		const uint64 DestSubSize = OutMipInfo.DataSize / DepthOrArraySize;
		const uint64 PaddingSubSize = DestSubSize - SourceSubSize;

		uint8* DestData = (uint8*)OutMipInfo.DestData;
		for (uint32 SubIdx = 0; SubIdx < DepthOrArraySize; ++SubIdx)
		{
			Ar.Serialize(DestData, SourceSubSize);
			DestData += SourceSubSize;
			FMemory::Memzero(DestData, PaddingSubSize);
			DestData += PaddingSubSize;
		}
		return true;
	}

	UE_LOG(LogTexture, Warning, TEXT("Mismatch between cached mip size and streaming buffer size for mip %d of %s: %" UINT64_FMT " KB / %" INT64_FMT " KB."),
		ResourceState.MaxNumLODs - MipIndex, *Context.Resource->GetTextureName().ToString(), OutMipInfo.DataSize / 1024, MipSize / 1024);
	FMemory::Memzero(OutMipInfo.DestData, OutMipInfo.DataSize);
	return false;
}

int32 FTexture2DMipDataProvider_DDC::GetMips(
	const FTextureUpdateContext& Context,
	int32 StartingMipIndex,
	const FTextureMipInfoArray& MipInfos, 
	const FTextureUpdateSyncOptions& SyncOptions)
{
	FTexturePlatformData* const* PtrPlatformData = const_cast<UTexture*>(Context.Texture)->GetRunningPlatformData();
	if (PtrPlatformData && *PtrPlatformData)
	{
		const FTexturePlatformData* PlatformData = *PtrPlatformData;
		const int32 LODBias = static_cast<int32>(Context.MipsView.GetData() - PlatformData->Mips.GetData());

		for (int32 MipIndex = StartingMipIndex; MipIndex < CurrentFirstLODIdx; ++MipIndex)
		{
			bool bSuccess = false;
			if (DDCBuffers[MipIndex])
			{
				// The result must be read from a memory reader!
				FMemoryReaderView Ar(DDCBuffers[MipIndex], true);
				if (SerializeMipInfo(Context, Ar, MipIndex, DDCBuffers[MipIndex].GetSize(), MipInfos[MipIndex]))
				{
					bSuccess = true;
				}
				DDCBuffers[MipIndex].Reset();
			}

			if (!bSuccess)
			{
				AdvanceTo(ETickState::CleanUp, ETickThread::Async);
				return MipIndex; // We failed at getting this mip. Cancel will be called.
			}
		}
	}
	else
	{
		UE_LOGF(LogTexture, Error, "Attempting to stream data that has not been generated yet for mips [%d, %d) of %ls.",
			PendingFirstLODIdx, CurrentFirstLODIdx, *Context.Texture->GetPathName());
	}

	AdvanceTo(ETickState::CleanUp, ETickThread::Async);
	return CurrentFirstLODIdx;
}

bool FTexture2DMipDataProvider_DDC::PollMips(const FTextureUpdateSyncOptions& SyncOptions)
{
	AdvanceTo(ETickState::CleanUp, ETickThread::Async);
	return true;
}

void FTexture2DMipDataProvider_DDC::CleanUp(const FTextureUpdateSyncOptions& SyncOptions)
{
	ReleaseDDCResources();
	AdvanceTo(ETickState::Done, ETickThread::None);
}

void FTexture2DMipDataProvider_DDC::Cancel(const FTextureUpdateSyncOptions& SyncOptions)
{
	ReleaseDDCResources();
}

FTextureMipDataProvider::ETickThread FTexture2DMipDataProvider_DDC::GetCancelThread() const
{
	return DDCBuffers.Num() ? ETickThread::Async : ETickThread::None;
}

void FTexture2DMipDataProvider_DDC::ReleaseDDCResources()
{
	DDCRequestOwner.Cancel();
	DDCBuffers.Empty();
}

#endif //WITH_EDITORONLY_DATA
