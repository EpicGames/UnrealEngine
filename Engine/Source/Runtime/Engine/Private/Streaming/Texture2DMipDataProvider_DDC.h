// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
Texture2DMipDataProvider_DDC.h : Implementation of FTextureMipDataProvider using DDC requests.
=============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "Memory/SharedBuffer.h"
#include "Streaming/TextureMipDataProvider.h"
#include "UObject/WeakObjectPtr.h"

#if WITH_EDITORONLY_DATA
#include "DerivedDataRequestOwner.h"

/**
* FTexture2DMipDataProvider_DDC implements FTextureMipAllocator using DDC requests from FDerivedDataCacheInterface.
*/
class FTexture2DMipDataProvider_DDC : public FTextureMipDataProvider
{
public:

	FTexture2DMipDataProvider_DDC(UTexture* Texture);
	~FTexture2DMipDataProvider_DDC();


	// ********************************************************
	// ********* FTextureMipDataProvider implementation **********
	// ********************************************************

	void Init(const FTextureUpdateContext& Context, const FTextureUpdateSyncOptions& SyncOptions) final override;
	int32 GetMips(const FTextureUpdateContext& Context, int32 StartingMipIndex, const FTextureMipInfoArray& MipInfos, const FTextureUpdateSyncOptions& SyncOptions) final override;
	bool PollMips(const FTextureUpdateSyncOptions& SyncOptions) final override;
	void CleanUp(const FTextureUpdateSyncOptions& SyncOptions) final override;
	void Cancel(const FTextureUpdateSyncOptions& SyncOptions) final override;
	ETickThread GetCancelThread() const final override;

protected:

	void ReleaseDDCResources();

	bool SerializeMipInfo(const FTextureUpdateContext& Context, FArchive& Ar, int32 MipIndex, int64 MipSize, const FTextureMipInfo& OutMipInfo);

private:

	TArray<FSharedBuffer, TInlineAllocator<MAX_TEXTURE_MIP_COUNT> > DDCBuffers;
	UE::DerivedData::FRequestOwner DDCRequestOwner;
	TWeakObjectPtr<UTexture> Texture;
};

#endif //WITH_EDITORONLY_DATA
