// Copyright Epic Games, Inc. All Rights Reserved.FTexture2DMipDataProvider_IO

/*=============================================================================
Texture2DMipDataProvider_IO.cpp : Implementation of FTextureMipDataProvider using cooked file IO.
=============================================================================*/

#include "Texture2DMipDataProvider_IO.h"
#include "Engine/Texture.h"
#include "HAL/PlatformFile.h"
#include "TextureResource.h"
#include "Streaming/TextureStreamingHelpers.h"
#include "ContentStreaming.h"
#include "ProfilingDebugging/IoStoreTrace.h"

FTexture2DMipDataProvider_IO::FTexture2DMipDataProvider_IO(const UTexture* InTexture, bool InPrioritizedIORequest)
	: FTextureMipDataProvider(InTexture, ETickState::Init, ETickThread::Async)
	, bPrioritizedIORequest(InPrioritizedIORequest)
{
	if (InTexture)
	{
		TextureName = InTexture->GetFName();
	}
}

FTexture2DMipDataProvider_IO::~FTexture2DMipDataProvider_IO()
{
	check(!IORequests.Num());
}

void FTexture2DMipDataProvider_IO::Init(const FTextureUpdateContext& Context, const FTextureUpdateSyncOptions& SyncOptions)
{
	IORequests.AddDefaulted(CurrentFirstLODIdx);

	// If this resource has optional LODs and we are streaming one of them.
	if (ResourceState.NumNonOptionalLODs < ResourceState.MaxNumLODs && PendingFirstLODIdx < ResourceState.LODCountToFirstLODIdx(ResourceState.NumNonOptionalLODs))
	{
		// Generate the FilenameHash of each optional LOD before the first one requested, so that we can handle properly PAK unmount events.
		// Note that streamer only stores the hash for the first optional mip.
		for (int32 MipIdx = 0; MipIdx < PendingFirstLODIdx; ++MipIdx)
		{
			const FTexture2DMipMap& OwnerMip = *Context.MipsView[MipIdx];
			IORequests[MipIdx].FilenameHash = OwnerMip.BulkData.GetIoFilenameHash();
		}
	}
	
	// Otherwise validate each streamed in mip.
	for (int32 MipIdx = PendingFirstLODIdx; MipIdx < CurrentFirstLODIdx; ++MipIdx)
	{
		const FTexture2DMipMap& OwnerMip = *Context.MipsView[MipIdx];
		if (OwnerMip.BulkData.IsStoredCompressedOnDisk())
		{
			// Compression at the package level is no longer supported
			continue;
		}
		else if (OwnerMip.BulkData.GetBulkDataSize() <= 0)
		{
			// Invalid bulk data size.
			continue;
		}
		else
		{
			IORequests[MipIdx].FilenameHash = OwnerMip.BulkData.GetIoFilenameHash();
		}
	}

	AdvanceTo(ETickState::GetMips, ETickThread::Async);
}

int32 FTexture2DMipDataProvider_IO::GetMips(
	const FTextureUpdateContext& Context, 
	int32 StartingMipIndex, 
	const FTextureMipInfoArray& MipInfos, 
	const FTextureUpdateSyncOptions& SyncOptions)
{
	SetAsyncFileCallback(SyncOptions);
	check(SyncOptions.Counter);
	
	while (StartingMipIndex < CurrentFirstLODIdx && MipInfos.IsValidIndex(StartingMipIndex))
	{
		const FTexture2DMipMap& OwnerMip = *Context.MipsView[StartingMipIndex];
		const FTextureMipInfo& MipInfo = MipInfos[StartingMipIndex];

		// Check the validity of the filename.
		if (IORequests[StartingMipIndex].FilenameHash == INVALID_IO_FILENAME_HASH)
		{
			break;
		}

		// If Data size is specified, check compatibility for safety
		if (MipInfo.DataSize && (uint64)OwnerMip.BulkData.GetBulkDataSize() > MipInfo.DataSize)
		{
			break;
		}
		
		// Ensure there is some valid place to read into.
		if (!MipInfo.DestData)
		{
			break;
		}

		// Increment as we push the requests. If a request completes immediately, then it will call the callback
		// but that won't do anything because the tick would not try to acquire the lock since it is already locked.
		SyncOptions.Counter->Increment();

		EAsyncIOPriorityAndFlags Priority = AIOP_Low;
		if (bPrioritizedIORequest)
		{
			static IConsoleVariable* CVarAsyncLoadingPrecachePriority = IConsoleManager::Get().FindConsoleVariable(TEXT("s.AsyncLoadingPrecachePriority"));
			const bool bLoadBeforeAsyncPrecache = CVarStreamingLowResHandlingMode.GetValueOnAnyThread() == (int32)FRenderAssetStreamingSettings::LRHM_LoadBeforeAsyncPrecache;

			if (CVarAsyncLoadingPrecachePriority && bLoadBeforeAsyncPrecache)
			{
				const int32 AsyncIOPriority = CVarAsyncLoadingPrecachePriority->GetInt();
				// Higher priority than regular requests but don't go over max
				Priority = (EAsyncIOPriorityAndFlags)FMath::Clamp<int32>(AsyncIOPriority + 1, AIOP_BelowNormal, AIOP_MAX);
			}
			else
			{
				Priority = AIOP_BelowNormal;
			}
		}

		TRACE_IOSTORE_METADATA_SCOPE_TAG("Texture2DMip");
		IORequests[StartingMipIndex].BulkDataIORequest.Reset(
			OwnerMip.BulkData.CreateStreamingRequest(
				0,
				OwnerMip.BulkData.GetBulkDataSize(),
				Priority | AIOP_FLAG_DONTCACHE,
				&AsyncFileCallBack,
				(uint8*)MipInfo.DestData
			)
		);

		++StartingMipIndex;
	}

	AdvanceTo(ETickState::PollMips, ETickThread::Async);
	return StartingMipIndex;
}

bool FTexture2DMipDataProvider_IO::PollMips(const FTextureUpdateSyncOptions& SyncOptions)
{
	// Notify that some files have possibly been unmounted / missing.
	if (bIORequestCancelled && !bIORequestAborted)
	{
		IRenderAssetStreamingManager& StreamingManager = IStreamingManager::Get().GetRenderAssetStreamingManager();
		for (FIORequest& IORequest : IORequests)
		{
			StreamingManager.MarkMountedStateDirty(IORequest.FilenameHash);
		}

		UE_LOGF(LogContentStreaming, Warning, "[%ls] Texture stream in request failed due to IO error (Mip %d-%d).", *TextureName.ToString(), ResourceState.AssetLODBias + PendingFirstLODIdx, ResourceState.AssetLODBias + CurrentFirstLODIdx - 1);
	}

	ClearIORequests();
	AdvanceTo(ETickState::Done, ETickThread::None);

	return !bIORequestCancelled;
}

void FTexture2DMipDataProvider_IO::AbortPollMips() 
{
	for (FIORequest& IORequest : IORequests)
	{
		if (IORequest.BulkDataIORequest)
		{
			// Calling cancel() here will trigger the AsyncFileCallBack and precipitate the execution of Cancel().
			IORequest.BulkDataIORequest->Cancel();
			bIORequestAborted = true;
		}
	}
}


void FTexture2DMipDataProvider_IO::CleanUp(const FTextureUpdateSyncOptions& SyncOptions)
{
	AdvanceTo(ETickState::Done, ETickThread::None);
}

void FTexture2DMipDataProvider_IO::Cancel(const FTextureUpdateSyncOptions& SyncOptions)
{
	ClearIORequests();
}

FTextureMipDataProvider::ETickThread FTexture2DMipDataProvider_IO::GetCancelThread() const
{
	return IORequests.Num() ? FTextureMipDataProvider::ETickThread::Async : FTextureMipDataProvider::ETickThread::None;
}

void FTexture2DMipDataProvider_IO::SetAsyncFileCallback(const FTextureUpdateSyncOptions& SyncOptions)
{
	FThreadSafeCounter* Counter = SyncOptions.Counter;
	FTextureUpdateSyncOptions::FCallback RescheduleCallback = SyncOptions.RescheduleCallback;
	check(Counter && RescheduleCallback);

	AsyncFileCallBack = [this, Counter, RescheduleCallback](bool bWasCancelled, IBulkDataIORequest* Req)
	{
		// At this point task synchronization would hold the number of pending requests.
		Counter->Decrement();
		
		if (bWasCancelled)
		{
			bIORequestCancelled = true;
		}

		if (Counter->GetValue() == 0)
		{
#if !UE_BUILD_SHIPPING
			// On some platforms the IO is too fast to test cancelation requests timing issues.
			if (FRenderAssetStreamingSettings::ExtraIOLatency > 0)
			{
				FPlatformProcess::Sleep(FRenderAssetStreamingSettings::ExtraIOLatency * .001f); // Slow down the streaming.
			}
#endif
			RescheduleCallback();
		}
	};
}

void FTexture2DMipDataProvider_IO::ClearIORequests()
{
	for (FIORequest& IORequest : IORequests)
	{
		// If requests are not yet completed, cancel and wait.
		if (IORequest.BulkDataIORequest && !IORequest.BulkDataIORequest->PollCompletion())
		{
			IORequest.BulkDataIORequest->Cancel();
			IORequest.BulkDataIORequest->WaitCompletion();
		}
	}
	IORequests.Empty();
}
