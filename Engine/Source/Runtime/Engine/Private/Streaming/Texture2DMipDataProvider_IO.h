// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
Texture2DMipDataProvider_IO.h : Implementation of FTextureMipDataProvider using cooked file IO.
=============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "Streaming/TextureMipDataProvider.h"
#include "Async/AsyncFileHandle.h"
#include "Serialization/BulkData.h"

/**
* FTexture2DMipDataProvider_IO implements FTextureMipAllocator using file IO (cooked content).
* It support having mips stored in different files contrary to FTexture2DStreamIn_IO.
*/
class FTexture2DMipDataProvider_IO : public FTextureMipDataProvider
{
public:

	FTexture2DMipDataProvider_IO(const UTexture* InTexture, bool InPrioritizedIORequest);
	~FTexture2DMipDataProvider_IO();

	// ********************************************************
	// ********* FTextureMipDataProvider implementation **********
	// ********************************************************

	void Init(const FTextureUpdateContext& Context, const FTextureUpdateSyncOptions& SyncOptions) final override;
	int32 GetMips(const FTextureUpdateContext& Context, int32 StartingMipIndex, const FTextureMipInfoArray& MipInfos, const FTextureUpdateSyncOptions& SyncOptions) final override;
	bool PollMips(const FTextureUpdateSyncOptions& SyncOptions) final override;
	void AbortPollMips() final override;
	void CleanUp(const FTextureUpdateSyncOptions& SyncOptions) final override;
	void Cancel(const FTextureUpdateSyncOptions& SyncOptions) final override;
	ETickThread GetCancelThread() const final override;

protected:

	// A structured with information about which file contains which mips.
	struct FIORequest
	{
		FIoFilenameHash FilenameHash = INVALID_IO_FILENAME_HASH;
		TUniquePtr<IBulkDataIORequest> BulkDataIORequest;
	};

	// Pending async requests created in GetMips().
	TArray<FIORequest, TInlineAllocator<MAX_TEXTURE_MIP_COUNT>> IORequests;

	// The asset name, used to log IO errors.
	FName TextureName;
	// Whether async read requests must be created with high priority (executes faster). 
	bool bPrioritizedIORequest = false;
	// Whether async read requests where cancelled for any reasons.
	bool bIORequestCancelled = false;
	// Whether async read requests were required to abort through AbortPollMips().
	bool bIORequestAborted = false;

	// A callback to be executed once all IO pending requests are completed.
	FBulkDataIORequestCallBack AsyncFileCallBack;

	// Helper to configure the AsyncFileCallBack.
	void SetAsyncFileCallback(const FTextureUpdateSyncOptions& SyncOptions);

	// Release / cancel any pending async file requests.
	void ClearIORequests();
};
