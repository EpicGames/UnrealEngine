// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
Texture2DStreamIn.cpp: Stream in helper for 2D textures.
=============================================================================*/

#include "Streaming/Texture2DStreamIn.h"
#include "EngineLogs.h"
#include "RenderUtils.h"
#include "Rendering/Texture2DResource.h"
#include "Streaming/Texture2DUpdate.h"

static int32 GStreamingForceUnlockMipsOnIOCancel = 1;
static FAutoConsoleVariableRef CVarStreamingForceUnlockMipsOnIOCancel(
	TEXT("r.Streaming.ForceUnlockMipsOnIOCancel"),
	GStreamingForceUnlockMipsOnIOCancel,
	TEXT("Whether to force unlock mips on IO Cancel."),
	ECVF_Default);

FTexture2DStreamIn::FTexture2DStreamIn(UTexture2D* InTexture)
	: FTexture2DUpdate(InTexture)
{
	ensure(ResourceState.NumRequestedLODs > ResourceState.NumResidentLODs);
	MipData.AddZeroed(ResourceState.MaxNumLODs);
}

FTexture2DStreamIn::~FTexture2DStreamIn()
{
#if DO_CHECK
	for (FStreamMipData & ThisMipData : MipData)
	{
		check(ThisMipData.Data == nullptr);
	}
#endif
}

void FTexture2DStreamIn::DoAllocateNewMips(const FContext& Context)
{
	if (!IsCancelled() && Context.Resource)
	{
		for (int32 MipIndex = PendingFirstLODIdx; MipIndex < CurrentFirstLODIdx; ++MipIndex)
		{
			const FTexture2DMipMap& MipMap = *Context.MipsView[MipIndex];
			const SIZE_T MipSize = CalcTextureMipMapSize(MipMap.SizeX, MipMap.SizeY, Context.Resource->GetPixelFormat(), 0);

			check(MipData[MipIndex].Data == nullptr);
			MipData[MipIndex].Data = FMemory::Malloc(MipSize);
			MipData[MipIndex].Size = MipSize;
			MipData[MipIndex].Pitch = 0; // 0 means tight packed
		}
	}
}

void FTexture2DStreamIn::DoFreeNewMips(const FContext& Context)
{
	for (int32 MipIndex = PendingFirstLODIdx; MipIndex < CurrentFirstLODIdx; ++MipIndex)
	{
		if (MipData[MipIndex].Data != nullptr)
		{
			FMemory::Free(MipData[MipIndex].Data);
			MipData[MipIndex].Data = nullptr;
			MipData[MipIndex].Size = 0;
			MipData[MipIndex].Pitch = -1;
		}
	}
}

void FTexture2DStreamIn::DoLockNewMips(FRHICommandListImmediate& RHICmdList, const FContext& Context)
{
	check(Context.CurrentThread == TT_Render);

	if (!IsCancelled() && IntermediateTextureRHI && Context.Resource)
	{
		// With virtual textures, all mips exist although they might not be allocated.
		const int32 MipOffset = !!(IntermediateTextureRHI->GetFlags() & TexCreate_Virtual) ? 0 : PendingFirstLODIdx;

		for (int32 MipIndex = PendingFirstLODIdx; MipIndex < CurrentFirstLODIdx; ++MipIndex)
		{
			check(MipData[MipIndex].Data == nullptr);
			uint32 DestPitch = -1;
			uint64 Size = ~0ULL;
			MipData[MipIndex].Data = RHICmdList.LockTexture2D(IntermediateTextureRHI, MipIndex - MipOffset, RLM_WriteOnly, DestPitch, false, CVarFlushRHIThreadOnSTreamingTextureLocks.GetValueOnAnyThread() > 0, &Size);
			MipData[MipIndex].Size = Size;
			MipData[MipIndex].Pitch = DestPitch;

			UE_LOGF(LogTextureUpload,Verbose,"FTexture2DStreamIn::DoLockNewMips( : Lock Mip %d Pitch=%d",MipIndex,DestPitch);
		}
	}
}


void FTexture2DStreamIn::DoUnlockNewMips(FRHICommandListImmediate& RHICmdList, const FContext& Context)
{
	check(Context.CurrentThread == TT_Render);

	if (IntermediateTextureRHI && (Context.Resource || (IsCancelled() && GStreamingForceUnlockMipsOnIOCancel)))
	{
		// With virtual textures, all mips exist although they might not be allocated.
		const int32 MipOffset = !!(IntermediateTextureRHI->GetFlags() & TexCreate_Virtual) ? 0 : PendingFirstLODIdx;

		for (int32 MipIndex = PendingFirstLODIdx; MipIndex < CurrentFirstLODIdx; ++MipIndex)
		{
			if (MipData[MipIndex].Data != nullptr)
			{
				RHICmdList.UnlockTexture2D(IntermediateTextureRHI, MipIndex - MipOffset, false, CVarFlushRHIThreadOnSTreamingTextureLocks.GetValueOnAnyThread() > 0 );
				MipData[MipIndex].Data = nullptr;
				MipData[MipIndex].Size = 0;
				MipData[MipIndex].Pitch = -1;
			}
		}
	}
}

void FTexture2DStreamIn::DoCopySharedMips(FRHICommandListImmediate& RHICmdList, const FContext& Context)
{
	check(Context.CurrentThread == TT_Render);

	if (!IsCancelled() && IntermediateTextureRHI && Context.Resource)
	{
		UE::RHI::CopySharedMips_AssumeSRVMaskState(
			RHICmdList,
			Context.Resource->GetTexture2DRHI(),
			IntermediateTextureRHI);
	}
}

// Async create the texture to the requested size.
void FTexture2DStreamIn::DoAsyncCreateWithNewMips(const FContext& Context)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FTexture2DStreamIn::DoAsyncCreateWithNewMips);
	check(Context.CurrentThread == TT_Async);

	if (!IsCancelled() && Context.Resource)
	{
		const FTexture2DMipMap& RequestedMipMap = *Context.MipsView[PendingFirstLODIdx];

		// old textures have sizes padded up to multiple of 4; that's wrong, should be real size unpadded
		check( RequestedMipMap.SizeX == FMath::Max(1, (int32)Context.MipsView[0]->SizeX >> PendingFirstLODIdx) );
		check( RequestedMipMap.SizeY == FMath::Max(1, (int32)Context.MipsView[0]->SizeY >> PendingFirstLODIdx) );

		check( PendingFirstLODIdx+ResourceState.NumRequestedLODs <= MipData.Num() );

		// RHIAsyncCreateTexture2D needs void ** array
		memset(InitialMipDataForAsyncCreate,0,sizeof(InitialMipDataForAsyncCreate));
		for(int i=0;i<MipData.Num();i++)
		{
			InitialMipDataForAsyncCreate[i] = MipData[i].Data;

			// this is only called with allocated mips that are tight-packed, not locked mips with pitches?
			check( MipData[i].Pitch == 0 );
		}
		
		// RHIAsyncCreateTexture2D assumes MipData is tight packed strides
		FTexture2DResource::WarnRequiresTightPackedMip(RequestedMipMap.SizeX,RequestedMipMap.SizeY,Context.Resource->GetPixelFormat(),MipData[PendingFirstLODIdx].Pitch);

		ensure(IntermediateTextureRHI == nullptr);
		FGraphEventRef CompletionEvent;
		IntermediateTextureRHI = RHIAsyncCreateTexture2D(
			RequestedMipMap.SizeX,
			RequestedMipMap.SizeY,
			Context.Resource->GetPixelFormat(),
			ResourceState.NumRequestedLODs,
			Context.Resource->GetCreationFlags(),
			ERHIAccess::Unknown,
			InitialMipDataForAsyncCreate+PendingFirstLODIdx,
			ResourceState.NumRequestedLODs - ResourceState.NumResidentLODs,
			*Context.Resource->GetTextureName().ToString(),
			CompletionEvent);

		if (CompletionEvent)
		{
			TaskSynchronization.Increment();
			FFunctionGraphTask::CreateAndDispatchWhenReady(
				[this]()
				{
					TaskSynchronization.Decrement();
				},
				TStatId{},
				CompletionEvent);
		}
	}
}

void FTexture2DStreamIn::ValidateMipBulkDataSize(const UTexture2D& Texture, int32 MipSizeX, int32 MipSizeY, int32 MipIndex, int64 ActualMipSize, int64& BulkDataSize)
{
	if (BulkDataSize != ActualMipSize)
	{
#if !UE_BUILD_SHIPPING
		UE_LOGF(LogTexture, Warning, "Mip (%d) %dx%d has an unexpected size %lld, expected size %lld. %ls, Pixel format %ls",
			MipIndex, MipSizeX, MipSizeY, BulkDataSize, ActualMipSize, *Texture.GetFullName(), GPixelFormats[Texture.GetPixelFormat()].Name);
#endif
		// Make sure we don't overrun buffer allocated for this mip
		BulkDataSize = FMath::Min(BulkDataSize, ActualMipSize);
	}
}