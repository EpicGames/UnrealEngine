// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
Texture2DStreamIn.h: Stream in helper for 2D textures.
=============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "Texture2DUpdate.h"

// Base StreamIn framework exposing MipData
class FTexture2DStreamIn : public FTexture2DUpdate
{
public:

	FTexture2DStreamIn(UTexture2D* InTexture);
	~FTexture2DStreamIn();

protected:

	// StreamIn_Default : Locked mips of the intermediate textures, used as disk load destination.
	struct FStreamMipData
	{
		void * Data = nullptr;
		uint64 Size = 0;
		uint32 Pitch = 0;
	};
	TArray<FStreamMipData, TInlineAllocator<MAX_TEXTURE_MIP_COUNT> > MipData;
	void * InitialMipDataForAsyncCreate[MAX_TEXTURE_MIP_COUNT];

	// ****************************
	// ********* Helpers **********
	// ****************************

	// Allocate memory for each mip.
	void DoAllocateNewMips(const FContext& Context);
	// Free allocated memory for each mip.
	void DoFreeNewMips(const FContext& Context);

	// Lock each streamed mips into MipData.
	void DoLockNewMips(FRHICommandListImmediate& RHICmdList, const FContext& Context);
	// Unlock each streamed mips from MipData.
	void DoUnlockNewMips(FRHICommandListImmediate& RHICmdList, const FContext& Context);

	// Copy each shared mip to the intermediate texture.
	void DoCopySharedMips(FRHICommandListImmediate& RHICmdList, const FContext& Context);

	// Async create the texture to the requested size.
	void DoAsyncCreateWithNewMips(const FContext& Context);

	// Make sure the destination mip buffer is large enough to store the bulk data, warn and adjust if not.
	void ValidateMipBulkDataSize(const UTexture2D& Texture, int32 MipSizeX, int32 MipSizeY, int32 MipIndex, int64 ActualMipSize, int64& BulkDataSize);
};
