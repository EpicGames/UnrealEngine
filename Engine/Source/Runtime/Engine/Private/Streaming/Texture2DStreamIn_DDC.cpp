// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
Texture2DStreamIn_DDC.cpp: Stream in helper for 2D textures loading DDC files.
=============================================================================*/

#include "Streaming/Texture2DStreamIn_DDC.h"
#include "EngineLogs.h"
#include "TextureCompiler.h"
#include "Rendering/Texture2DResource.h"
#include "RenderUtils.h"
#include "Streaming/Texture2DStreamIn.h"
#include "Streaming/TextureStreamingHelpers.h"
#include "Streaming/Texture2DUpdate.h"

#if WITH_EDITORONLY_DATA

#include "DerivedDataCache.h"

int32 GStreamingUseAsyncRequestsForDDC = 1;
static FAutoConsoleVariableRef CVarStreamingDDCPendingSleep(
	TEXT("r.Streaming.UseAsyncRequestsForDDC"),
	GStreamingUseAsyncRequestsForDDC,
	TEXT("Whether to use async DDC requests in order to react quickly to cancel and suspend rendering requests (default=0)"),
	ECVF_Default
);

// ******************************************
// ********* FTexture2DStreamIn_DDC *********
// ******************************************

FTexture2DStreamIn_DDC::FTexture2DStreamIn_DDC(UTexture2D* InTexture)
	: FTexture2DStreamIn(InTexture)
	, DDCRequestOwner(UE::DerivedData::EPriority::Normal)
	, Texture(InTexture)
{
	DDCMipRequestStatus.AddZeroed(ResourceState.MaxNumLODs);
}

FTexture2DStreamIn_DDC::~FTexture2DStreamIn_DDC()
{
}

void FTexture2DStreamIn_DDC::DoCreateAsyncDDCRequests(const FContext& Context)
{
	if (!Context.Texture || !Context.Resource)
	{
		return;
	}

	if (const FTexturePlatformData* PlatformData = Context.Texture->GetPlatformData())
	{
		const int32 LODBias = static_cast<int32>(Context.MipsView.GetData() - PlatformData->Mips.GetData());

		using namespace UE::DerivedData;
		TArray<FCacheGetValueRequest> DDC1MipKeys;
		TArray<FCacheGetChunkRequest> DDC2MipKeys;

		TStringBuilder<256> MipNameBuilder;
		Context.Texture->GetPathName(nullptr, MipNameBuilder);
		const int32 TextureNameLen = MipNameBuilder.Len();

		const bool bUsingDDC1 = PlatformData->DerivedDataKey.IsType<FString>();
		const bool bUsingDDC2 = PlatformData->DerivedDataKey.IsType<UE::DerivedData::FCacheKeyProxy>();
		if (bUsingDDC1)
		{
			for (int32 MipIndex = PendingFirstLODIdx; MipIndex < CurrentFirstLODIdx && !IsCancelled(); ++MipIndex)
			{
				const FTexture2DMipMap& MipMap = *Context.MipsView[MipIndex];
				FMipRequestStatus& Status = DDCMipRequestStatus[MipIndex];
				if (!MipMap.IsPagedToDerivedData())
				{
					UE_LOGF(LogTexture, Error, "Attempting to stream data that is already loaded for mip %d of %ls.",
						MipIndex, *Context.Texture->GetPathName());
					MarkAsCancelled();
				}
				else if (!Status.bRequestIssued && !Status.Buffer)
				{
					FCacheGetValueRequest& Request = DDC1MipKeys.AddDefaulted_GetRef();
					MipNameBuilder.Appendf(TEXT(" [MIP %d]"), MipIndex + LODBias);
					Request.Name = MipNameBuilder;
					Request.Key = ConvertLegacyCacheKey(PlatformData->GetDerivedDataMipKeyString(MipIndex + LODBias, MipMap));
					Request.UserData = MipIndex;
					MipNameBuilder.RemoveSuffix(MipNameBuilder.Len() - TextureNameLen);
					Status.bRequestIssued = true;
				}
			}
		}
		else if (bUsingDDC2)
		{
			const FCacheKey& Key = *PlatformData->DerivedDataKey.Get<UE::DerivedData::FCacheKeyProxy>().AsCacheKey();
			for (int32 MipIndex = PendingFirstLODIdx; MipIndex < CurrentFirstLODIdx && !IsCancelled(); ++MipIndex)
			{
				const FTexture2DMipMap& MipMap = *Context.MipsView[MipIndex];
				FMipRequestStatus& Status = DDCMipRequestStatus[MipIndex];
				if (MipMap.IsPagedToDerivedData() && !Status.bRequestIssued && !Status.Buffer)
				{
					FCacheGetChunkRequest& Request = DDC2MipKeys.AddDefaulted_GetRef();
					MipNameBuilder.Appendf(TEXT(" [MIP %d]"), MipIndex + LODBias);
					Request.Name = MipNameBuilder;
					Request.Key = Key;
					Request.Id = FTexturePlatformData::MakeMipId(MipIndex + LODBias);
					Request.UserData = MipIndex;
					MipNameBuilder.RemoveSuffix(MipNameBuilder.Len() - TextureNameLen);
					Status.bRequestIssued = true;
				}
			}
		}
		else
		{
			UE_LOGF(LogTexture, Error, "Attempting to stream data in an unsupported cache format for mips [%d, %d) of %ls.",
				PendingFirstLODIdx, CurrentFirstLODIdx, *Context.Texture->GetPathName());
			MarkAsCancelled();
		}

		if (DDC1MipKeys.Num() || DDC2MipKeys.Num())
		{
		#if !UE_BUILD_SHIPPING
			// On some platforms the IO is too fast to test cancellation requests timing issues.
			if (FRenderAssetStreamingSettings::ExtraIOLatency > 0 && TaskSynchronization.GetValue() == 0)
			{
				const int32 NumMipKeys = FMath::Max(DDC1MipKeys.Num(), DDC2MipKeys.Num());
				FPlatformProcess::Sleep(NumMipKeys * FRenderAssetStreamingSettings::ExtraIOLatency * 0.001f); // Slow down the streaming.
			}
		#endif

			FRequestBarrier Barrier(DDCRequestOwner);

			if (bUsingDDC1)
			{
				GetCache().GetValue(DDC1MipKeys, DDCRequestOwner, [this](FCacheGetValueResponse&& Response)
					{
						bool bOk = Response.Status == EStatus::Ok;
						if (bOk)
						{
							if (FSharedBuffer&& MipBuffer = Response.Value.GetData().Decompress())
							{
								const int32 MipIndex = int32(Response.UserData);
								FMipRequestStatus& Status = DDCMipRequestStatus[MipIndex];
								check(!Status.Buffer);
								Status.Buffer = MipBuffer;
								check(Status.bRequestIssued);
								Status.bRequestIssued = false;
							}
							else
							{
								bOk = false;
							}
						}

						if (!bOk)
						{
							FTextureCompilingManager::Get().ForceDeferredTextureRebuildAnyThread({ Texture });
						}
					});
			}
			else if (bUsingDDC2)
			{
				GetCache().GetChunks(DDC2MipKeys, DDCRequestOwner, [this](FCacheGetChunkResponse&& Response)
					{
						if (Response.Status == EStatus::Ok)
						{
							const int32 MipIndex = int32(Response.UserData);
							FMipRequestStatus& Status = DDCMipRequestStatus[MipIndex];
							check(!Status.Buffer);
							Status.Buffer = MoveTemp(Response.RawData);
							check(Status.bRequestIssued);
							Status.bRequestIssued = false;
						}
						else if (Response.Status == EStatus::Error)
						{
							FTextureCompilingManager::Get().ForceDeferredTextureRebuildAnyThread({ Texture });
						}
					});
			}
		}
	}
	else
	{
		UE_LOGF(LogTexture, Error, "Attempting to stream data that has not been generated yet for mips [%d, %d) of %ls.",
			PendingFirstLODIdx, CurrentFirstLODIdx, *Context.Texture->GetPathName());
		MarkAsCancelled();
	}
}

bool FTexture2DStreamIn_DDC::DoPoolDDCRequests(const FContext& Context) 
{
	return DDCRequestOwner.Poll();
}

void FTexture2DStreamIn_DDC::DoLoadNewMipsFromDDC(const FContext& Context)
{
	if (!Context.Texture || !Context.Resource)
	{
		return;
	}

	using namespace UE::DerivedData;
	const EPriority OriginalPriority = DDCRequestOwner.GetPriority();
	ON_SCOPE_EXIT { DDCRequestOwner.SetPriority(OriginalPriority); };
	DDCRequestOwner.SetPriority(EPriority::Blocking);
	DoCreateAsyncDDCRequests(Context);
	DDCRequestOwner.Wait();

	for (int32 MipIndex = PendingFirstLODIdx; MipIndex < CurrentFirstLODIdx && !IsCancelled(); ++MipIndex)
	{
		const FTexture2DMipMap& MipMap = *Context.MipsView[MipIndex];
		check(MipData[MipIndex].Data);

		FMipRequestStatus& Status = DDCMipRequestStatus[MipIndex];
		if (Status.Buffer)
		{
			const SIZE_T ExpectedMipSize = CalcTextureMipMapSize(MipMap.SizeX, MipMap.SizeY, Context.Resource->GetPixelFormat(), 0);

			UE_LOG(LogTexture, Verbose, TEXT("FTexture2DStreamIn_DDC::DoLoadNewMipsFromDDC Size=%dx%d ExpectedMipSize=%" SIZE_T_FMT " DerivedMipSize=%" SIZE_T_FMT),
				MipMap.SizeX, MipMap.SizeY, ExpectedMipSize, Status.Buffer.GetSize());

			// Serializes directly into MipData so it must be tight packed pitch
			const uint32 DestPitch = MipData[MipIndex].Pitch;
			FTexture2DResource::WarnRequiresTightPackedMip(MipMap.SizeX, MipMap.SizeY, Context.Resource->GetPixelFormat(), DestPitch);

			if (Status.Buffer.GetSize() == ExpectedMipSize)
			{
				// Pitch ignored
				// to copy and respect Pitch, use CopyTextureData2D
				check(MipData[MipIndex].Size >= Status.Buffer.GetSize());
				FMemory::Memcpy(MipData[MipIndex].Data, Status.Buffer.GetData(), Status.Buffer.GetSize());
			}
			else
			{
				UE_LOG(LogTexture, Error, TEXT("Cached mip size (%" SIZE_T_FMT ") not as expected (%" SIZE_T_FMT ") for mip %d of %s."),
					static_cast<SIZE_T>(Status.Buffer.GetSize()), ExpectedMipSize, MipIndex, *Context.Texture->GetPathName());
				MarkAsCancelled();
			}
		}
		else
		{
			MarkAsCancelled();
		}
	}

	FPlatformMisc::MemoryBarrier();
}

#endif
