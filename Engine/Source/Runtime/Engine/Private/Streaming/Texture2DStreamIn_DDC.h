// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
Texture2DStreamIn_DDC.h: Stream in helper for 2D textures loading DDC files.
=============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "Memory/SharedBuffer.h"
#include "Texture2DStreamIn.h"
#include "UObject/WeakObjectPtr.h"

#if WITH_EDITORONLY_DATA
#include "DerivedDataRequestOwner.h"

extern int32 GStreamingUseAsyncRequestsForDDC;

class FTexture2DStreamIn_DDC : public FTexture2DStreamIn
{
public:

	FTexture2DStreamIn_DDC(UTexture2D* InTexture);
	~FTexture2DStreamIn_DDC();

protected:

	struct FMipRequestStatus
	{
		FSharedBuffer Buffer;
		bool bRequestIssued = false;
	};

	TArray<FMipRequestStatus, TInlineAllocator<MAX_TEXTURE_MIP_COUNT> > DDCMipRequestStatus;
	UE::DerivedData::FRequestOwner DDCRequestOwner;
	TWeakObjectPtr<UTexture2D> Texture;

	// ****************************
	// ********* Helpers **********
	// ****************************

	// Create DDC load requests
	void DoCreateAsyncDDCRequests(const FContext& Context);

	// Create DDC load requests
	bool DoPoolDDCRequests(const FContext& Context);

	// Load from DDC into MipData
	void DoLoadNewMipsFromDDC(const FContext& Context);
};

#endif // WITH_EDITORONLY_DATA
