// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
Texture2DStreamIn_DDC_AsyncCreate.h: Load texture 2D mips from the DDC using async create.
=============================================================================*/

#include "Streaming/Texture2DStreamIn_DDC_AsyncCreate.h"
#include "Streaming/Texture2DStreamIn_DDC.h"
#include "Streaming/Texture2DUpdate.h"

#if WITH_EDITORONLY_DATA

FTexture2DStreamIn_DDC_AsyncCreate::FTexture2DStreamIn_DDC_AsyncCreate(UTexture2D* InTexture)
	: FTexture2DStreamIn_DDC(InTexture) 
{
	if (GStreamingUseAsyncRequestsForDDC)
	{
		PushTask(FContext(InTexture, TT_None), TT_Async, SRA_UPDATE_CALLBACK(AsyncDDC), TT_None, nullptr);
	}
	else
	{
		PushTask(FContext(InTexture, TT_None), TT_Async, SRA_UPDATE_CALLBACK(AllocateAndLoadMips), TT_None, nullptr);
	}
}

// ****************************
// ******* Update Steps *******
// ****************************

void FTexture2DStreamIn_DDC_AsyncCreate::AsyncDDC(const FContext& Context)
{
	DECLARE_SCOPE_CYCLE_COUNTER(TEXT("FTexture2DStreamIn_DDC_AsyncCreate::AsyncDDC"), STAT_Texture2DStreamInDDCAsyncCreate_AsyncDDC, STATGROUP_StreamingDetails);
	check(Context.CurrentThread == TT_Async);

	DoCreateAsyncDDCRequests(Context);

	bDeferExecution = !IsCancelled(); // Set as pending.
	PushTask(Context, TT_Async, SRA_UPDATE_CALLBACK(PollDDC), TT_Render, SRA_UPDATE_CALLBACK(Cancel));
}

void FTexture2DStreamIn_DDC_AsyncCreate::PollDDC(const FContext& Context)
{
	DECLARE_SCOPE_CYCLE_COUNTER(TEXT("FTexture2DStreamIn_DDC_AsyncCreate::PollDDC"), STAT_Texture2DStreamInDDCAsyncCreate_PollDDC, STATGROUP_StreamingDetails);
	check(Context.CurrentThread == TT_Async);

	if (DoPoolDDCRequests(Context))
	{
		PushTask(Context, TT_Async, SRA_UPDATE_CALLBACK(AllocateAndLoadMips), TT_Render, SRA_UPDATE_CALLBACK(Cancel));
	}
	else
	{
		bDeferExecution = !IsCancelled(); // Set as pending.
		PushTask(Context, TT_Async, SRA_UPDATE_CALLBACK(PollDDC), TT_Render, SRA_UPDATE_CALLBACK(Cancel));
	}
}

void FTexture2DStreamIn_DDC_AsyncCreate::AllocateAndLoadMips(const FContext& Context)
{
	DECLARE_SCOPE_CYCLE_COUNTER(TEXT("FTexture2DStreamIn_DDC_AsyncCreate::AllocateAndLoadMips"), STAT_Texture2DStreamInDDCAsyncCreate_AllocateAndLoadMips, STATGROUP_StreamingDetails);
	check(Context.CurrentThread == TT_Async);

	DoAllocateNewMips(Context);
	DoLoadNewMipsFromDDC(Context);

	PushTask(Context, TT_Async, SRA_UPDATE_CALLBACK(AsyncCreate), TT_Render, SRA_UPDATE_CALLBACK(Cancel));
}

void FTexture2DStreamIn_DDC_AsyncCreate::AsyncCreate(const FContext& Context)
{
	DECLARE_SCOPE_CYCLE_COUNTER(TEXT("FTexture2DStreamIn_DDC_AsyncCreate::AsyncCreate"), STAT_Texture2DStreamInDDCAsyncCreate_AsyncCreate, STATGROUP_StreamingDetails);
	check(Context.CurrentThread == TT_Async);

	DoAsyncCreateWithNewMips(Context);
	DoFreeNewMips(Context);

	PushTask(Context, TT_Render, SRA_UPDATE_CALLBACK(Finalize), TT_Render, SRA_UPDATE_CALLBACK(Cancel));
}

void FTexture2DStreamIn_DDC_AsyncCreate::Finalize(const FContext& Context)
{
	DECLARE_SCOPE_CYCLE_COUNTER(TEXT("FTexture2DStreamIn_DDC_AsyncCreate::Finalize"), STAT_Texture2DStreamInDDCAsyncCreate_Finalize, STATGROUP_StreamingDetails);
	check(Context.CurrentThread == TT_Render);

	FRHICommandListImmediate& RHICmdList = FRHICommandListImmediate::Get();

	DoCopySharedMips(RHICmdList, Context);
	DoFinishUpdate(RHICmdList, Context);
}

// ****************************
// ******* Cancel Steps *******
// ****************************


void FTexture2DStreamIn_DDC_AsyncCreate::Cancel(const FContext& Context)
{
	DECLARE_SCOPE_CYCLE_COUNTER(TEXT("FTexture2DStreamIn_DDC_AsyncCreate::Cancel"), STAT_Texture2DStreamInDDCAsyncCreate_Cancel, STATGROUP_StreamingDetails);
	check(Context.CurrentThread == TT_Render);

	FRHICommandListImmediate& RHICmdList = FRHICommandListImmediate::Get();

	DoFreeNewMips(Context);
	DoFinishUpdate(RHICmdList, Context);
}

#endif // WITH_EDITORONLY_DATA
