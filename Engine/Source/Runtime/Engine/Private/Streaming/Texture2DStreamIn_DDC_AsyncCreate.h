// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
Texture2DStreamIn_DDC_AsyncCreate.h: Load texture 2D mips from the DDC using async create.
=============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "Texture2DStreamIn_DDC.h"

#if WITH_EDITORONLY_DATA

class FTexture2DStreamIn_DDC_AsyncCreate : public FTexture2DStreamIn_DDC
{
public:

	FTexture2DStreamIn_DDC_AsyncCreate(UTexture2D* InTexture);

protected:

	// ****************************
	// ******* Update Steps *******
	// ****************************

	// Create DDC requests for each mips. (AsyncThread)
	void AsyncDDC(const FContext& Context);
	// Poll DDC requests completion for each mips. (AsyncThread)
	void PollDDC(const FContext& Context);
	// Allocate the MipData (AsyncThread)
	void AllocateAndLoadMips(const FContext& Context);
	// Create load requests into each locked mips. (AsyncThread)
	void AsyncCreate(const FContext& Context);
	// Apply the intermediate texture and cleanup. (RenderThread)
	void Finalize(const FContext& Context);

	// ****************************
	// ******* Cancel Steps *******
	// ****************************

	// Cancel the update. (RenderThread)
	void Cancel(const FContext& Context);
};

#endif // WITH_EDITORONLY_DATA
