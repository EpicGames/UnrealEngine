// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
Texture2DStreamIn_DDC_AsyncReallocate.cpp: Load texture 2D mips from the DDC using async reallocate.
=============================================================================*/

#include "Streaming/Texture2DStreamIn_DDC_AsyncReallocate.h"
#include "Streaming/Texture2DStreamIn_DDC.h"
#include "Streaming/Texture2DUpdate.h"

#if WITH_EDITORONLY_DATA

FTexture2DStreamIn_DDC_AsyncReallocate::FTexture2DStreamIn_DDC_AsyncReallocate(UTexture2D* InTexture)
	: FTexture2DStreamIn_DDC(InTexture)
{
	PushTask(FContext(InTexture, TT_None), TT_Render, SRA_UPDATE_CALLBACK(AsyncReallocate), TT_None, nullptr);
}

// ****************************
// ******* Update Steps *******
// ****************************

void FTexture2DStreamIn_DDC_AsyncReallocate::AsyncReallocate(const FContext& Context)
{
	DECLARE_SCOPE_CYCLE_COUNTER(TEXT("FTexture2DStreamIn_DDC_AsyncReallocate::AsyncReallocate"), STAT_Texture2DStreamInDDCAsyncReallocate_AsyncReallocate, STATGROUP_StreamingDetails);
	check(Context.CurrentThread == TT_Render);

	FRHICommandListImmediate& RHICmdList = FRHICommandListImmediate::Get();

	DoAsyncReallocate(RHICmdList, Context);

	PushTask(Context, TT_Render, SRA_UPDATE_CALLBACK(LockMips), TT_Render, SRA_UPDATE_CALLBACK(Cancel));
}

void FTexture2DStreamIn_DDC_AsyncReallocate::LockMips(const FContext& Context)
{
	DECLARE_SCOPE_CYCLE_COUNTER(TEXT("FTexture2DStreamIn_DDC_AsyncReallocate::LockMips"), STAT_Texture2DStreamInDDCAsyncReallocate_LockMips, STATGROUP_StreamingDetails);
	check(Context.CurrentThread == TT_Render);

	FRHICommandListImmediate& RHICmdList = FRHICommandListImmediate::Get();

	DoLockNewMips(RHICmdList, Context);

	PushTask(Context, TT_Async, SRA_UPDATE_CALLBACK(LoadMips), TT_Render, SRA_UPDATE_CALLBACK(Cancel));
}

void FTexture2DStreamIn_DDC_AsyncReallocate::LoadMips(const FContext& Context)
{
	DECLARE_SCOPE_CYCLE_COUNTER(TEXT("FTexture2DStreamIn_DDC_AsyncReallocate::LoadMips"), STAT_Texture2DStreamInDDCAsyncReallocate_LoadMips, STATGROUP_StreamingDetails);
	check(Context.CurrentThread == TT_Async);

	DoLoadNewMipsFromDDC(Context);

	PushTask(Context, TT_Render, SRA_UPDATE_CALLBACK(Finalize), TT_Render, SRA_UPDATE_CALLBACK(Cancel));
}

void FTexture2DStreamIn_DDC_AsyncReallocate::Finalize(const FContext& Context)
{
	DECLARE_SCOPE_CYCLE_COUNTER(TEXT("FTexture2DStreamIn_DDC_AsyncReallocate::Finalize"), STAT_Texture2DStreamInDDCAsyncReallocate_Finalize, STATGROUP_StreamingDetails);
	check(Context.CurrentThread == TT_Render);

	FRHICommandListImmediate& RHICmdList = FRHICommandListImmediate::Get();

	DoUnlockNewMips(RHICmdList, Context);
	DoFinishUpdate(RHICmdList, Context);
}

// ****************************
// ******* Cancel Steps *******
// ****************************

void FTexture2DStreamIn_DDC_AsyncReallocate::Cancel(const FContext& Context)
{
	DECLARE_SCOPE_CYCLE_COUNTER(TEXT("FTexture2DStreamIn_DDC_AsyncReallocate::Cancel"), STAT_Texture2DStreamInDDCAsyncReallocate_Cancel, STATGROUP_StreamingDetails);
	check(Context.CurrentThread == TT_Render);

	FRHICommandListImmediate& RHICmdList = FRHICommandListImmediate::Get();

	DoUnlockNewMips(RHICmdList, Context);
	DoFinishUpdate(RHICmdList, Context);
}

#endif // WITH_EDITORONLY_DATA
