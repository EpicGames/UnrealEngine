// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
Texture2DStreamIn_DDC_AsyncReallocate.h: Load texture 2D mips from the DDC using async reallocate.
=============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "Texture2DStreamIn_DDC.h"

#if WITH_EDITORONLY_DATA

class FTexture2DStreamIn_DDC_AsyncReallocate : public FTexture2DStreamIn_DDC
{
public:

	FTexture2DStreamIn_DDC_AsyncReallocate(UTexture2D* InTexture);

protected:

	// ****************************
	// ******* Update Steps *******
	// ****************************

	// Create an intermediate bigger copy of the texture. (RenderThread)
	void AsyncReallocate(const FContext& Context);
	// Lock each new mips of the intermediate texture. (RenderThread)
	void LockMips(const FContext& Context);
	// Create load requests into each locked mips. (AsyncThread)
	void LoadMips(const FContext& Context);
	// Unlock the mips, apply the intermediate texture and clean up. (RenderThread)
	void Finalize(const FContext& Context);

	// ****************************
	// ******* Cancel Steps *******
	// ****************************

	// Unlock the mips, cancel load requests and clean up. (RenderThread)
	void Cancel(const FContext& Context);
};

#endif // WITH_EDITORONLY_DATA
