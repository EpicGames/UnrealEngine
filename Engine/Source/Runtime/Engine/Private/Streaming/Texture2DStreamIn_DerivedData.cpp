// Copyright Epic Games, Inc. All Rights Reserved.

#include "Streaming/Texture2DStreamIn_DerivedData.h"

#include "EngineLogs.h"
#include "Rendering/Texture2DResource.h"
#include "Streaming/TextureStreamingHelpers.h"
#include "Streaming/Texture2DStreamIn.h"
#include "Streaming/Texture2DUpdate.h"

namespace UE
{

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

FTexture2DStreamIn_DerivedData::FTexture2DStreamIn_DerivedData(UTexture2D* InTexture, bool bInHighPriority)
	: FTexture2DStreamIn(InTexture)
	, bHighPriority(bInHighPriority)
{
}

void FTexture2DStreamIn_DerivedData::DoBeginIoRequests(const FContext& Context)
{
	TaskSynchronization.Increment();

	FDerivedDataIoBatch Batch;

	for (int32 MipIndex = PendingFirstLODIdx; MipIndex < CurrentFirstLODIdx && !IsCancelled(); ++MipIndex)
	{
		const FTexture2DMipMap& MipMap = *Context.MipsView[MipIndex];
		check(MipData[MipIndex].Data);

		if (MipMap.DerivedData.HasData())
		{
			uint32 ExpectedPitch = 0;
			int64 MipSize = FTexture2DResource::CalculateTightPackedMipSize(MipMap.SizeX, MipMap.SizeY, Context.Resource->GetPixelFormat(), ExpectedPitch);

			// Validate buffer size for the mip, so we don't overrun it on streaming
			ValidateMipBulkDataSize(*Context.Texture, MipMap.SizeX, MipMap.SizeY, MipIndex, MipData[MipIndex].Size, MipSize);

			FDerivedDataIoOptions Options(MakeMemoryView(MipData[MipIndex].Data, MipSize));
			Batch.Read(MipMap.DerivedData, Options);

			// Pitch is ignored because this reads directly into MipData[].
			// The Batch.Dispatch callback could be used to fix Pitch.
			FTexture2DResource::WarnRequiresTightPackedMip(MipMap.SizeX, MipMap.SizeY, Context.Resource->GetPixelFormat(), MipData[MipIndex].Pitch);
		}
		else if (MipMap.BulkData.IsBulkDataLoaded())
		{
			UE_LOGF(LogTexture, Error, "Attempting to stream data for non-streaming mip %d of %ls.",
				MipIndex, *Context.Texture->GetPathName());
			MarkAsCancelled();
		}
		else
		{
			UE_LOGF(LogTexture, Error, "Attempting to stream data that has not yet been generated for mip %d of %ls.",
				MipIndex, *Context.Texture->GetPathName());
			MarkAsCancelled();
		}
	}

	FDerivedDataIoPriority Priority = FDerivedDataIoPriority::Low().InterpolateTo(FDerivedDataIoPriority::Normal(), bHighPriority ? 0.5f : 0.0f);

	Batch.Dispatch(Response, Priority, [this]
	{
		TaskSynchronization.Decrement();

		if (Response.GetOverallStatus() != EDerivedDataIoStatus::Ok)
		{
			MarkAsCancelled();
		}

		if constexpr (!UE_BUILD_SHIPPING)
		{
			if (const int32 ExtraLatencyMs = FRenderAssetStreamingSettings::ExtraIOLatency; ExtraLatencyMs > 0)
			{
				// Simulate extra latency to allow enough time to cancel requests even when I/O is very fast.
				FPlatformProcess::Sleep((CurrentFirstLODIdx - PendingFirstLODIdx) * ExtraLatencyMs * 0.001f);
			}
		}

		ResponseComplete.Notify();

		// Tick to schedule the task or cancellation callback. Use TT_None to avoid a deadlock.
		Tick(FTexture2DUpdate::TT_None);
	});
}

bool FTexture2DStreamIn_DerivedData::DoPollIoRequests(const FContext& Context)
{
	return Response.Poll();
}

void FTexture2DStreamIn_DerivedData::DoEndIoRequests(const FContext& Context)
{
	ResponseComplete.Wait();
}

void FTexture2DStreamIn_DerivedData::DoCancelIoRequests()
{
	Response.Cancel();
}

void FTexture2DStreamIn_DerivedData::Abort()
{
	FTexture2DStreamIn::Abort();
	DoCancelIoRequests();
}

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

FTexture2DStreamIn_DerivedData_AsyncCreate::FTexture2DStreamIn_DerivedData_AsyncCreate(UTexture2D* InTexture, bool bInHighPriority)
	: FTexture2DStreamIn_DerivedData(InTexture, bInHighPriority)
{
	const FContext Context(InTexture, TT_None);
	PushTask(Context, TT_Async, SRA_UPDATE_CALLBACK(AllocateMipsAndBeginIoRequests), TT_None, nullptr);
}

void FTexture2DStreamIn_DerivedData_AsyncCreate::AllocateMipsAndBeginIoRequests(const FContext& Context)
{
	DECLARE_SCOPE_CYCLE_COUNTER(TEXT("FTexture2DStreamIn_DerivedData_AsyncCreate::AllocateMipsAndBeginIoRequests"),
		STAT_Texture2DStreamIn_DerivedData_AsyncCreate_AllocateMipsAndBeginIoRequests, STATGROUP_StreamingDetails);
	check(Context.CurrentThread == TT_Async);

	DoAllocateNewMips(Context);
	DoBeginIoRequests(Context);

	bDeferExecution = !IsCancelled(); // Set as pending.
	PushTask(Context, TT_Async, SRA_UPDATE_CALLBACK(PollIoRequests), TT_Render, SRA_UPDATE_CALLBACK(Cancel));
}

void FTexture2DStreamIn_DerivedData_AsyncCreate::PollIoRequests(const FContext& Context)
{
	DECLARE_SCOPE_CYCLE_COUNTER(TEXT("FTexture2DStreamIn_DerivedData_AsyncCreate::PollIoRequests"),
		STAT_Texture2DStreamIn_DerivedData_AsyncCreate_PollIoRequests, STATGROUP_StreamingDetails);
	check(Context.CurrentThread == TT_Async);

	if (!DoPollIoRequests(Context))
	{
		bDeferExecution = !IsCancelled(); // Set as pending.
		PushTask(Context, TT_Async, SRA_UPDATE_CALLBACK(PollIoRequests), TT_Render, SRA_UPDATE_CALLBACK(Cancel));
	}
	else
	{
		PushTask(Context, TT_Async, SRA_UPDATE_CALLBACK(EndIoRequests), TT_Render, SRA_UPDATE_CALLBACK(Cancel));
	}
}

void FTexture2DStreamIn_DerivedData_AsyncCreate::EndIoRequests(const FContext& Context)
{
	DECLARE_SCOPE_CYCLE_COUNTER(TEXT("FTexture2DStreamIn_DerivedData_AsyncCreate::EndIoRequests"),
		STAT_Texture2DStreamIn_DerivedData_AsyncCreate_EndIoRequests, STATGROUP_StreamingDetails);
	check(Context.CurrentThread == TT_Async);

	DoEndIoRequests(Context);

	PushTask(Context, TT_Async, SRA_UPDATE_CALLBACK(AsyncCreate), TT_Render, SRA_UPDATE_CALLBACK(Cancel));
}

void FTexture2DStreamIn_DerivedData_AsyncCreate::AsyncCreate(const FContext& Context)
{
	DECLARE_SCOPE_CYCLE_COUNTER(TEXT("FTexture2DStreamIn_DerivedData_AsyncCreate::AsyncCreate"),
		STAT_Texture2DStreamIn_DerivedData_AsyncCreate_AsyncCreate, STATGROUP_StreamingDetails);
	check(Context.CurrentThread == TT_Async);

	DoAsyncCreateWithNewMips(Context);
	DoFreeNewMips(Context);

	PushTask(Context, TT_Render, SRA_UPDATE_CALLBACK(Finalize), TT_Render, SRA_UPDATE_CALLBACK(Cancel));
}

void FTexture2DStreamIn_DerivedData_AsyncCreate::Finalize(const FContext& Context)
{
	DECLARE_SCOPE_CYCLE_COUNTER(TEXT("FTexture2DStreamIn_DerivedData_AsyncCreate::Finalize"),
		STAT_Texture2DStreamIn_DerivedData_AsyncCreate_Finalize, STATGROUP_StreamingDetails);
	check(Context.CurrentThread == TT_Render);

	FRHICommandListImmediate& RHICmdList = FRHICommandListImmediate::Get();

	DoCopySharedMips(RHICmdList, Context);
	DoFinishUpdate(RHICmdList, Context);
}

void FTexture2DStreamIn_DerivedData_AsyncCreate::Cancel(const FContext& Context)
{
	DECLARE_SCOPE_CYCLE_COUNTER(TEXT("FTexture2DStreamIn_DerivedData_AsyncCreate::Cancel"),
		STAT_Texture2DStreamIn_DerivedData_AsyncCreate_Cancel, STATGROUP_StreamingDetails);
	check(Context.CurrentThread == TT_Render);

	FRHICommandListImmediate& RHICmdList = FRHICommandListImmediate::Get();

	DoCancelIoRequests();
	DoEndIoRequests(Context);

	DoFreeNewMips(Context);
	DoFinishUpdate(RHICmdList, Context);
}

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

FTexture2DStreamIn_DerivedData_AsyncReallocate::FTexture2DStreamIn_DerivedData_AsyncReallocate(UTexture2D* InTexture, bool bInHighPriority)
	: FTexture2DStreamIn_DerivedData(InTexture, bInHighPriority)
{
	const FContext Context(InTexture, TT_None);
	PushTask(Context, TT_Render, SRA_UPDATE_CALLBACK(AsyncReallocate), TT_None, nullptr);
}

void FTexture2DStreamIn_DerivedData_AsyncReallocate::AsyncReallocate(const FContext& Context)
{
	DECLARE_SCOPE_CYCLE_COUNTER(TEXT("FTexture2DStreamIn_DerivedData_AsyncReallocate::AsyncReallocate"),
		STAT_Texture2DStreamIn_DerivedData_AsyncReallocate_AsyncReallocate, STATGROUP_StreamingDetails);
	check(Context.CurrentThread == TT_Render);

	FRHICommandListImmediate& RHICmdList = FRHICommandListImmediate::Get();

	DoAsyncReallocate(RHICmdList, Context);

	PushTask(Context, TT_Render, SRA_UPDATE_CALLBACK(LockMips), TT_Render, SRA_UPDATE_CALLBACK(Cancel));
}

void FTexture2DStreamIn_DerivedData_AsyncReallocate::LockMips(const FContext& Context)
{
	DECLARE_SCOPE_CYCLE_COUNTER(TEXT("FTexture2DStreamIn_DerivedData_AsyncReallocate::LockMips"),
		STAT_Texture2DStreamIn_DerivedData_AsyncReallocate_LockMips, STATGROUP_StreamingDetails);
	check(Context.CurrentThread == TT_Render);

	FRHICommandListImmediate& RHICmdList = FRHICommandListImmediate::Get();

	DoLockNewMips(RHICmdList, Context);

	PushTask(Context, TT_Async, SRA_UPDATE_CALLBACK(LoadMips), TT_Render, SRA_UPDATE_CALLBACK(Cancel));
}

void FTexture2DStreamIn_DerivedData_AsyncReallocate::LoadMips(const FContext& Context)
{
	DECLARE_SCOPE_CYCLE_COUNTER(TEXT("FTexture2DStreamIn_DerivedData_AsyncReallocate::LoadMips"),
		STAT_Texture2DStreamIn_DerivedData_AsyncReallocate_LoadMips, STATGROUP_StreamingDetails);
	check(Context.CurrentThread == TT_Async);

	DoBeginIoRequests(Context);
	DoEndIoRequests(Context);

	PushTask(Context, TT_Render, SRA_UPDATE_CALLBACK(Finalize), TT_Render, SRA_UPDATE_CALLBACK(Cancel));
}

void FTexture2DStreamIn_DerivedData_AsyncReallocate::Finalize(const FContext& Context)
{
	DECLARE_SCOPE_CYCLE_COUNTER(TEXT("FTexture2DStreamIn_DerivedData_AsyncReallocate::Finalize"),
		STAT_Texture2DStreamIn_DerivedData_AsyncReallocate_Finalize, STATGROUP_StreamingDetails);
	check(Context.CurrentThread == TT_Render);

	FRHICommandListImmediate& RHICmdList = FRHICommandListImmediate::Get();

	DoUnlockNewMips(RHICmdList, Context);
	DoFinishUpdate(RHICmdList, Context);
}

void FTexture2DStreamIn_DerivedData_AsyncReallocate::Cancel(const FContext& Context)
{
	DECLARE_SCOPE_CYCLE_COUNTER(TEXT("FTexture2DStreamIn_DerivedData_AsyncReallocate::Cancel"),
		STAT_Texture2DStreamIn_DerivedData_AsyncReallocate_Cancel, STATGROUP_StreamingDetails);
	check(Context.CurrentThread == TT_Render);

	FRHICommandListImmediate& RHICmdList = FRHICommandListImmediate::Get();

	DoCancelIoRequests();
	DoEndIoRequests(Context);

	DoUnlockNewMips(RHICmdList, Context);
	DoFinishUpdate(RHICmdList, Context);
}

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

FTexture2DStreamIn_DerivedData_Virtual::FTexture2DStreamIn_DerivedData_Virtual(UTexture2D* InTexture, bool bInHighPriority)
	: FTexture2DStreamIn_DerivedData(InTexture, bInHighPriority)
{
	PushTask(FContext(InTexture, TT_None), TT_Render, SRA_UPDATE_CALLBACK(LockMips), TT_None, nullptr);
}

void FTexture2DStreamIn_DerivedData_Virtual::LockMips(const FContext& Context)
{
	DECLARE_SCOPE_CYCLE_COUNTER(TEXT("FTexture2DStreamIn_DerivedData_Virtual::LockMips"), STAT_Texture2DStreamIn_DerivedData_Virtual_LockMips, STATGROUP_StreamingDetails);
	check(Context.CurrentThread == TT_Render);

	FRHICommandListImmediate& RHICmdList = FRHICommandListImmediate::Get();

	DoConvertToVirtualWithNewMips(RHICmdList, Context);
	DoLockNewMips(RHICmdList, Context);

	PushTask(Context, TT_Async, SRA_UPDATE_CALLBACK(LoadMips), TT_Render, SRA_UPDATE_CALLBACK(Cancel));
}

void FTexture2DStreamIn_DerivedData_Virtual::LoadMips(const FContext& Context)
{
	DECLARE_SCOPE_CYCLE_COUNTER(TEXT("FTexture2DStreamIn_DerivedData_Virtual::LoadMips"), STAT_Texture2DStreamIn_DerivedData_Virtual_LoadMips, STATGROUP_StreamingDetails);
	check(Context.CurrentThread == TT_Async);

	DoBeginIoRequests(Context);

	PushTask(Context, TT_Async, SRA_UPDATE_CALLBACK(PostLoadMips), TT_Render, SRA_UPDATE_CALLBACK(Cancel));
}

void FTexture2DStreamIn_DerivedData_Virtual::PostLoadMips(const FContext& Context)
{
	DECLARE_SCOPE_CYCLE_COUNTER(TEXT("FTexture2DStreamIn_DerivedData_Virtual::PostLoadMips"), STAT_Texture2DStreamIn_DerivedData_Virtual_LoadMips, STATGROUP_StreamingDetails);
	check(Context.CurrentThread == TT_Async);

	DoEndIoRequests(Context);

	PushTask(Context, TT_Render, SRA_UPDATE_CALLBACK(Finalize), TT_Render, SRA_UPDATE_CALLBACK(Cancel));
}


void FTexture2DStreamIn_DerivedData_Virtual::Finalize(const FContext& Context)
{
	DECLARE_SCOPE_CYCLE_COUNTER(TEXT("FTexture2DStreamIn_DerivedData_Virtual::Finalize"), STAT_Texture2DStreamIn_DerivedData_Virtual_Finalize, STATGROUP_StreamingDetails);
	check(Context.CurrentThread == TT_Render);

	FRHICommandListImmediate& RHICmdList = FRHICommandListImmediate::Get();

	DoUnlockNewMips(RHICmdList, Context);
	if (IntermediateTextureRHI)
	{
		RHICmdList.VirtualTextureSetFirstMipVisible(IntermediateTextureRHI, PendingFirstLODIdx);
	}
	DoFinishUpdate(RHICmdList, Context);
}

void FTexture2DStreamIn_DerivedData_Virtual::Cancel(const FContext& Context)
{
	DECLARE_SCOPE_CYCLE_COUNTER(TEXT("FTexture2DStreamIn_DerivedData_Virtual::Cancel"), STAT_Texture2DStreamIn_DerivedData_Virtual_Cancel, STATGROUP_StreamingDetails);
	check(Context.CurrentThread == TT_Render);

	FRHICommandListImmediate& RHICmdList = FRHICommandListImmediate::Get();

	DoCancelIoRequests();
	DoEndIoRequests(Context);

	DoUnlockNewMips(RHICmdList, Context);
	if (IntermediateTextureRHI)
	{
		RHICmdList.VirtualTextureSetFirstMipInMemory(IntermediateTextureRHI, CurrentFirstLODIdx);
	}
	DoFinishUpdate(RHICmdList, Context);
}

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

} // UE
