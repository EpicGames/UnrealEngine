// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Async/ManualResetEvent.h"
#include "Serialization/DerivedData.h"
#include "Texture2DStreamIn.h"

namespace UE
{

class FTexture2DStreamIn_DerivedData : public FTexture2DStreamIn
{
protected:
	FTexture2DStreamIn_DerivedData(UTexture2D* Texture, bool bHighPriority);

	/** Begin the async requests to load the pending mips. */
	void DoBeginIoRequests(const FContext& Context);

	/** Poll the async requests and return true if they are complete. */
	bool DoPollIoRequests(const FContext& Context);

	/** Wait until the async requests are complete and clean them up. */
	void DoEndIoRequests(const FContext& Context);

	/** Cancel any active async requests. Does not wait! */
	void DoCancelIoRequests();

	/** Cancel any active async requests. Does not wait! */
	void Abort() final;

private:
	FDerivedDataIoResponse Response;
	FManualResetEvent ResponseComplete;
	bool bHighPriority;
};

class FTexture2DStreamIn_DerivedData_AsyncCreate final : public FTexture2DStreamIn_DerivedData
{
public:
	FTexture2DStreamIn_DerivedData_AsyncCreate(UTexture2D* Texture, bool bHighPriority);

protected:
	// Allocate mips and begin the I/O request for each mip. (AsyncThread)
	void AllocateMipsAndBeginIoRequests(const FContext& Context);
	// Poll the I/O requests. (AsyncThread)
	void PollIoRequests(const FContext& Context);
	// Wait for the I/O requests to complete and clean them up. (AsyncThread)
	void EndIoRequests(const FContext& Context);
	// Async create an intermediate texture from the loaded mips. (AsyncThread)
	void AsyncCreate(const FContext& Context);
	// Apply the intermediate texture and clean up. (RenderThread)
	void Finalize(const FContext& Context);
	// Cancel the update. (RenderThread)
	void Cancel(const FContext& Context);
};

class FTexture2DStreamIn_DerivedData_AsyncReallocate final : public FTexture2DStreamIn_DerivedData
{
public:
	FTexture2DStreamIn_DerivedData_AsyncReallocate(UTexture2D* Texture, bool bHighPriority);

protected:
	// Create an intermediate bigger copy of the texture. (RenderThread)
	void AsyncReallocate(const FContext& Context);
	// Lock each new mips of the intermediate texture. (RenderThread)
	void LockMips(const FContext& Context);
	// Create load requests into each locked mips. (AsyncThread)
	void LoadMips(const FContext& Context);
	// Unlock the mips, apply the intermediate texture and clean up. (RenderThread)
	void Finalize(const FContext& Context);
	// Unlock the mips, cancel load requests and clean up. (RenderThread)
	void Cancel(const FContext& Context);
};

class FTexture2DStreamIn_DerivedData_Virtual final : public FTexture2DStreamIn_DerivedData
{
public:
	FTexture2DStreamIn_DerivedData_Virtual(UTexture2D* Texture, bool bHighPriority);

protected:
	// Validate the filename and bulk data, then lock new mips. (RenderThread)
	void LockMips(const FContext& Context);
	// Create load requests into each locked mips. (AsyncThread)
	void LoadMips(const FContext& Context);
	// Clear the IO requests and call Finalize. (AsyncThread)
	void PostLoadMips(const FContext& Context);
	// Apply the intermediate texture and cleanup. (RenderThread)
	void Finalize(const FContext& Context);
	// Unlock the mips, cancel load requests and clean up. (RenderThread)
	void Cancel(const FContext& Context);
};

} // UE
