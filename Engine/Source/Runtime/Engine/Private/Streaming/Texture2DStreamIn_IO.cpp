// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
Texture2DStreamIn.cpp: Stream in helper for 2D textures using texture streaming files.
=============================================================================*/

#include "Streaming/Texture2DStreamIn_IO.h"
#include "HAL/PlatformFile.h"
#include "Streaming/TextureStreamingHelpers.h"
#include "ContentStreaming.h"
#include "IO/IoBuffer.h"
#include "Rendering/Texture2DResource.h"
#include "Streaming/Texture2DStreamIn.h"
#include "Streaming/Texture2DUpdate.h"
#include "ProfilingDebugging/CsvProfiler.h"
#include "ProfilingDebugging/IoStoreTrace.h"
#include "EngineLogs.h"

CSV_DECLARE_CATEGORY_EXTERN(TextureStreaming);

#if !UE_BUILD_SHIPPING
float GTextureStreamInAbandonCallbackIOLatency = 0.0f;
static FAutoConsoleVariableRef CVarTextureStreamInFastCancelCallbackIOLatency(
	TEXT("r.Streaming.TextureStreamInAbandonCallbackIOLatency"),
	GTextureStreamInAbandonCallbackIOLatency,
	TEXT("Texture stream in latency (in ms), only applied to the bulk request callkbacks when abandons are enabled, useful to induce a case where CanAbandon is called while the IO request is in flight"),
	ECVF_Default);
#endif // #if !UE_BUILD_SHIPPING

FTexture2DStreamIn_IO::FTexture2DStreamIn_IO(UTexture2D* InTexture, bool InPrioritizedIORequest)
	: FTexture2DStreamIn(InTexture)
	, bPrioritizedIORequest(InPrioritizedIORequest)

{
}

FTexture2DStreamIn_IO::~FTexture2DStreamIn_IO()
{
}

static void ValidateMipBulkDataSize(const UTexture2D& Texture, int32 MipSizeX, int32 MipSizeY, int32 MipIndex, int64 ActualMipSize, int64& BulkDataSize)
{
	if (BulkDataSize != ActualMipSize)
	{
#if !UE_BUILD_SHIPPING
		UE_LOGF(LogTexture, Warning, "Mip (%d) %dx%d has an unexpected size %lld, expected size %lld. %ls, Pixel format %ls", 
			MipIndex, MipSizeX, MipSizeY, BulkDataSize, ActualMipSize, *Texture.GetFullName(), GPixelFormats[Texture.GetPixelFormat()].Name);
#endif
		// Make sure we don't overrun buffer allocated for this mip
		BulkDataSize = FMath::Min(BulkDataSize, ActualMipSize);
	}
}

void FTexture2DStreamIn_IO::SetIORequests(const FContext& Context)
{
	int32 InFlightBulkSize = 0;
	const int32 BatchCount = FMath::Min(CurrentFirstLODIdx - PendingFirstLODIdx, static_cast<int32>(ResourceState.MaxNumLODs));
	FBulkDataBatchRequest::FBatchBuilder Batch = FBulkDataBatchRequest::NewBatch(BatchCount);
	for (int32 MipIndex = PendingFirstLODIdx; MipIndex < CurrentFirstLODIdx && !IsCancelled(); ++MipIndex)
	{
		const FTexture2DMipMap& MipMap = *Context.MipsView[MipIndex];
		check(MipData[MipIndex].Data != nullptr);

		int64 BulkDataSize = MipMap.BulkData.GetBulkDataSize();
		if (BulkDataSize > 0)
		{
			// Validate buffer size for the mip, so we don't overrun it on streaming
			ValidateMipBulkDataSize(*Context.Texture, MipMap.SizeX, MipMap.SizeY, MipIndex, MipData[MipIndex].Size, BulkDataSize);
			
			// reads directly into MipData[] , doesn't respect Pitch
			// we do get a completion callback at AsyncFileCallBack
			// so in theory could fix Pitch there
			uint32 DestPitch = MipData[MipIndex].Pitch;
			FTexture2DResource::WarnRequiresTightPackedMip(MipMap.SizeX, MipMap.SizeY, Context.Resource->GetPixelFormat(), DestPitch);

			EAsyncIOPriorityAndFlags Priority = AIOP_Low;
			if (bPrioritizedIORequest)
			{
				static IConsoleVariable* CVarAsyncLoadingPrecachePriority = IConsoleManager::Get().FindConsoleVariable(TEXT("s.AsyncLoadingPrecachePriority"));
				const bool bLoadBeforeAsyncPrecache = CVarStreamingLowResHandlingMode.GetValueOnAnyThread() == (int32)FRenderAssetStreamingSettings::LRHM_LoadBeforeAsyncPrecache;

				if (CVarAsyncLoadingPrecachePriority && bLoadBeforeAsyncPrecache)
				{
					const int32 AsyncIOPriority = CVarAsyncLoadingPrecachePriority->GetInt();
					// Higher priority than regular requests but don't go over max
					Priority = (EAsyncIOPriorityAndFlags)FMath::Clamp<int32>(AsyncIOPriority + 1, AIOP_BelowNormal, AIOP_MAX);
				}
				else
				{
					Priority = AIOP_BelowNormal;
				}
			}


			TRACE_IOSTORE_METADATA_SCOPE_TAG(*StreamableAsset->GetName());
			FIoBuffer Dst(FIoBuffer::Wrap, MipData[MipIndex].Data, BulkDataSize);
			Batch.Read(MipMap.BulkData, 0, BulkDataSize, Priority | AIOP_FLAG_DONTCACHE | AIOP_FLAG_HW_TARGET_MEMORY, Dst);

			InFlightBulkSize += BulkDataSize;
		}
		else // Bulk data size can only be 0 when not available, in which case, we need to recache the file state.
		{
			bFailedOnIOError = true;
			MarkAsCancelled();
			break;
		}
	}

	if (bFailedOnIOError || Batch.IsEmpty())
	{
		return;
	}

	TaskSynchronization.Increment();

	Batch.Issue([this, InFlightBulkSize](FBulkDataBatchRequest::EStatus Status)
	{
#if !UE_BUILD_SHIPPING
		// This is separate from FRenderAssetStreamingSettings::ExtraIOLatency to make it easier to test abandons.
		// FRenderAssetStreamingSettings::ExtraIOLatency is applied to many parts of the asset update that can in turn stall GC.
		// Stalled GCs would not trigger the call to CanAbandon, testing which is the purpose of this CVar.
		if (GTextureStreamInAbandonCallbackIOLatency > 0)
		{
			FPlatformProcess::Sleep(GTextureStreamInAbandonCallbackIOLatency * .001f); // Slow down the streaming.
		}
#endif // !UE_BUILD_SHIPPING

		TaskSynchronization.Decrement();

		if (Status != FBulkDataBatchRequest::EStatus::Ok && !bIsCancelled)
		{
			// If IO requests was cancelled but the streaming request wasn't, this is an IO error.
			bFailedOnIOError = true;
			MarkAsCancelled();
		}
#if !UE_BUILD_SHIPPING
		// On some platforms the IO is too fast to test cancelation requests timing issues.
		if (FRenderAssetStreamingSettings::ExtraIOLatency > 0 && TaskSynchronization.GetValue() == 0)
		{
			FPlatformProcess::Sleep(FRenderAssetStreamingSettings::ExtraIOLatency * .001f); // Slow down the streaming.
		}
#endif
		if (bIsCancelled)
		{
			CSV_CUSTOM_STAT(TextureStreaming, CancelledMipsInMB, float(InFlightBulkSize) / (1024.0f * 1024.0f), ECsvCustomStatOp::Accumulate);
		}
		// The tick here is intended to schedule the success or cancel callback.
		// Using TT_None ensure gets which could create a dead lock.
		Tick(FTexture2DUpdate::TT_None);
	},
	BatchRequest);
}

void FTexture2DStreamIn_IO::CancelIORequests()
{
	if (BatchRequest.IsPending())
	{
		BatchRequest.Cancel();
	}
}

void FTexture2DStreamIn_IO::ClearIORequests(const FContext& Context)
{
	BatchRequest.Reset();
}

void FTexture2DStreamIn_IO::ReportIOError(const FContext& Context)
{
	// Invalidate the cache state of all initial mips (note that when using FIoChunkId each mip has a different value).
	if (bFailedOnIOError && Context.Texture)
	{
		IRenderAssetStreamingManager& StreamingManager = IStreamingManager::Get().GetTextureStreamingManager();
		// Need to start at index 0 because the streamer only gets the hash for the first optional mip (and we don't know which one it is).
		for (int32 MipIndex = 0; MipIndex < CurrentFirstLODIdx; ++MipIndex)
		{
			StreamingManager.MarkMountedStateDirty(Context.Texture->GetMipIoFilenameHash(ResourceState.AssetLODBias + MipIndex));
		}
		UE_LOGF(LogContentStreaming, Warning, "[%ls] Texture stream in request failed due to IO error (Mip %d-%d).", *Context.Texture->GetName(), ResourceState.AssetLODBias + PendingFirstLODIdx, ResourceState.AssetLODBias + CurrentFirstLODIdx - 1);
	}
}

void FTexture2DStreamIn_IO::Abort()
{
	if (!IsCancelled() && !IsCompleted())
	{
		FTexture2DStreamIn::Abort();

		if (BatchRequest.IsPending())
		{
			// Prevent the update from being considered done before this is finished.
			// By checking that it was not already canceled, we make sure this doesn't get called twice.
			(new FAsyncCancelIORequestsTask(this))->StartBackgroundTask();
		}
	}
}

bool FTexture2DStreamIn_IO::CanAbandon() const
{
	check(GEnableAssetUpdateAbandons);
	// Abandon if we have any in flight IO.
	return (FTexture2DStreamIn::CanAbandon()) 
			|| (!IsCompleted() && IsCancelled() && BatchRequest.IsPending() && TaskSynchronization.GetValue() > 0);
}

void FTexture2DStreamIn_IO::FCancelIORequestsTask::DoWork()
{
	check(PendingUpdate);
	// Acquire the lock of this object in order to cancel any pending IO.
	// If the object is currently being ticked, wait.
	const ETaskState PreviousTaskState = PendingUpdate->DoLock();
	PendingUpdate->CancelIORequests();
	PendingUpdate->DoUnlock(PreviousTaskState);
}
