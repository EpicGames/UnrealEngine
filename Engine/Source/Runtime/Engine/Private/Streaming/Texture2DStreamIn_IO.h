// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
Texture2DStreamIn.h: Stream in helper for 2D textures using texture streaming files.
=============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "Texture2DStreamIn.h"

class FBulkDataRequest;

// Base StreamIn framework exposing MipData
class FTexture2DStreamIn_IO : public FTexture2DStreamIn
{
public:

	FTexture2DStreamIn_IO(UTexture2D* InTexture, bool InPrioritizedIORequest);
	~FTexture2DStreamIn_IO();

protected:

	// ****************************
	// ********* Helpers **********
	// ****************************

	// Set the IO requests for streaming the mips.
	void SetIORequests(const FContext& Context);
	// Cancel / destroy each requests created in SetIORequests()
	void ClearIORequests(const FContext& Context);
	// Report IO errors if any.
	void ReportIOError(const FContext& Context);
	// Set the IO callback used for streaming the mips.
	void SetAsyncFileCallback() { };
	// Cancel all IO requests.
	void CancelIORequests();

	// Start an async task to cancel pending IO requests.
	void Abort() override;

	bool CanAbandon() const override;

private:

	class FCancelIORequestsTask : public FNonAbandonableTask
	{
	public:
		FCancelIORequestsTask(FTexture2DStreamIn_IO* InPendingUpdate) : PendingUpdate(InPendingUpdate) {}
		void DoWork();
		FORCEINLINE TStatId GetStatId() const
		{
			RETURN_QUICK_DECLARE_CYCLE_STAT(FCancelIORequestsTask_Texture, STATGROUP_ThreadPoolAsyncTasks);
		}
	protected:
		TRefCountPtr<FTexture2DStreamIn_IO> PendingUpdate;
	};

	typedef FAutoDeleteAsyncTask<FCancelIORequestsTask> FAsyncCancelIORequestsTask;
	friend class FCancelIORequestsTask;

	// Whether an IO error was detected (when files do not exists).
	bool bFailedOnIOError = false;

	// Whether the IO request should be created with an higher priority for quicker response time.
	bool bPrioritizedIORequest = false;

	// I/O request handle
	FBulkDataBatchRequest BatchRequest;
};

