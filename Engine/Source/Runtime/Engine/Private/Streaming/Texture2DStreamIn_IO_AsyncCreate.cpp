// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
Texture2DStreamIn_IO_AsyncCreate.cpp: Async create path for streaming in texture 2D mips.
=============================================================================*/

#include "Streaming/Texture2DStreamIn_IO_AsyncCreate.h"
#include "Streaming/Texture2DStreamIn_IO.h"
#include "Streaming/Texture2DUpdate.h"

FTexture2DStreamIn_IO_AsyncCreate::FTexture2DStreamIn_IO_AsyncCreate(UTexture2D* InTexture, bool InPrioritizedIORequest)
	: FTexture2DStreamIn_IO(InTexture, InPrioritizedIORequest) 
{
	PushTask(FContext(InTexture, TT_None), TT_Async, SRA_UPDATE_CALLBACK(AllocateAndLoadMips), TT_None, nullptr);
}

// ****************************
// ******* Update Steps *******
// ****************************

void FTexture2DStreamIn_IO_AsyncCreate::AllocateAndLoadMips(const FContext& Context)
{
	DECLARE_SCOPE_CYCLE_COUNTER(TEXT("FTexture2DStreamIn_IO_AsyncCreate::AllocateAndLoadMips"), STAT_Texture2DStreamInIOAsyncCreate_AllocateAndLoadMips, STATGROUP_StreamingDetails);
	check(Context.CurrentThread == TT_Async);

	DoAllocateNewMips(Context);
	SetIORequests(Context);

	PushTask(Context, TT_Async, SRA_UPDATE_CALLBACK(AsyncCreate), TT_Async, SRA_UPDATE_CALLBACK(CancelIO));
}


void FTexture2DStreamIn_IO_AsyncCreate::AsyncCreate(const FContext& Context)
{
	DECLARE_SCOPE_CYCLE_COUNTER(TEXT("FTexture2DStreamIn_IO_AsyncCreate::AsyncCreate"), STAT_Texture2DStreamInIOAsyncCreate_AsyncCreate, STATGROUP_StreamingDetails);
	check(Context.CurrentThread == TT_Async);

	DoAsyncCreateWithNewMips(Context);
	DoFreeNewMips(Context);
	ClearIORequests(Context);

	PushTask(Context, TT_Render, SRA_UPDATE_CALLBACK(Finalize), TT_Render, SRA_UPDATE_CALLBACK(Cancel));
}

void FTexture2DStreamIn_IO_AsyncCreate::Finalize(const FContext& Context)
{
	DECLARE_SCOPE_CYCLE_COUNTER(TEXT("FTexture2DStreamIn_IO_AsyncCreate::Finalize"), STAT_Texture2DStreamInIOAsyncCreate_Finalize, STATGROUP_StreamingDetails);
	check(Context.CurrentThread == TT_Render);

	FRHICommandListImmediate& RHICmdList = FRHICommandListImmediate::Get();

	DoCopySharedMips(RHICmdList, Context);
	DoFinishUpdate(RHICmdList, Context);
}

// ****************************
// ******* Cancel Steps *******
// ****************************

void FTexture2DStreamIn_IO_AsyncCreate::CancelIO(const FContext& Context)
{
	DECLARE_SCOPE_CYCLE_COUNTER(TEXT("FTexture2DStreamIn_IO_AsyncCreate::CancelIO"), STAT_Texture2DStreamInIOAsyncCreate_CancelIO, STATGROUP_StreamingDetails);
	check(Context.CurrentThread == TT_Async);

	ClearIORequests(Context);

	PushTask(Context, TT_None, nullptr, TT_Render, SRA_UPDATE_CALLBACK(Cancel));
}


void FTexture2DStreamIn_IO_AsyncCreate::Cancel(const FContext& Context)
{
	DECLARE_SCOPE_CYCLE_COUNTER(TEXT("FTexture2DStreamIn_IO_AsyncCreate::Cancel"), STAT_Texture2DStreamInIOAsyncCreate_Cancel, STATGROUP_StreamingDetails);
	check(Context.CurrentThread == TT_Render);

	FRHICommandListImmediate& RHICmdList = FRHICommandListImmediate::Get();

	DoFreeNewMips(Context);
	DoFinishUpdate(RHICmdList, Context);
	ReportIOError(Context);
}
