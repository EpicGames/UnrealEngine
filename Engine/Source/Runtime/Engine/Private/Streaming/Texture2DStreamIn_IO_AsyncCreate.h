// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
Texture2DStreamIn_IO_AsyncCreate.h: Async create path for streaming in texture 2D mips.
=============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "Texture2DStreamIn_IO.h"

class FTexture2DStreamIn_IO_AsyncCreate : public FTexture2DStreamIn_IO
{
public:

	FTexture2DStreamIn_IO_AsyncCreate(UTexture2D* InTexture, bool InPrioritizedIORequest);

protected:


	// ****************************
	// ******* Update Steps *******
	// ****************************

	// Allocate the MipData (AsyncThread)
	void AllocateAndLoadMips(const FContext& Context);
	// Create load requests into each locked mips. (AsyncThread)
	void AsyncCreate(const FContext& Context);
	// Apply the intermediate texture and cleanup. (RenderThread)
	void Finalize(const FContext& Context);

	// ****************************
	// ******* Cancel Steps *******
	// ****************************

	// Cancel pending IO then cancel (Async)
	void CancelIO(const FContext& Context);
	// Cancel the update. (RenderThread)
	void Cancel(const FContext& Context);

};
