// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
Texture2DStreamIn_IO_AsyncReallocate.cpp: Default path for streaming in texture 2D mips.
=============================================================================*/

#include "Streaming/Texture2DStreamIn_IO_AsyncReallocate.h"
#include "Streaming/Texture2DStreamIn_IO.h"
#include "Streaming/Texture2DUpdate.h"

FTexture2DStreamIn_IO_AsyncReallocate::FTexture2DStreamIn_IO_AsyncReallocate(UTexture2D* InTexture, bool InPrioritizedIORequest) 
	: FTexture2DStreamIn_IO(InTexture, InPrioritizedIORequest)
{
	PushTask(FContext(InTexture, TT_None), TT_Render, SRA_UPDATE_CALLBACK(AsyncReallocate), TT_None, nullptr);
}

// ****************************
// ******* Update Steps *******
// ****************************

void FTexture2DStreamIn_IO_AsyncReallocate::AsyncReallocate(const FContext& Context)
{
	DECLARE_SCOPE_CYCLE_COUNTER(TEXT("FTexture2DStreamIn_IO_AsyncReallocate::AsyncReallocate"), STAT_Texture2DStreamInIOAsyncReallocate_AsyncReallocate, STATGROUP_StreamingDetails);
	check(Context.CurrentThread == TT_Render);

	FRHICommandListImmediate& RHICmdList = FRHICommandListImmediate::Get();

	DoAsyncReallocate(RHICmdList, Context);

	PushTask(Context, TT_Render, SRA_UPDATE_CALLBACK(LockMips), TT_Render, SRA_UPDATE_CALLBACK(Cancel));
}

void FTexture2DStreamIn_IO_AsyncReallocate::LockMips(const FContext& Context)
{
	DECLARE_SCOPE_CYCLE_COUNTER(TEXT("FTexture2DStreamIn_IO_AsyncReallocate::LockMips"), STAT_Texture2DStreamInIOAsyncReallocate_LockMips, STATGROUP_StreamingDetails);
	check(Context.CurrentThread == TT_Render);

	FRHICommandListImmediate& RHICmdList = FRHICommandListImmediate::Get();

	DoLockNewMips(RHICmdList, Context);

	PushTask(Context, TT_Async, SRA_UPDATE_CALLBACK(LoadMips), TT_Render, SRA_UPDATE_CALLBACK(Cancel));
}

void FTexture2DStreamIn_IO_AsyncReallocate::LoadMips(const FContext& Context)
{
	DECLARE_SCOPE_CYCLE_COUNTER(TEXT("FTexture2DStreamIn_IO_AsyncReallocate::LoadMips"), STAT_Texture2DStreamInIOAsyncReallocate_LoadMips, STATGROUP_StreamingDetails);
	check(Context.CurrentThread == TT_Async);

	SetIORequests(Context);

	PushTask(Context, TT_Render, SRA_UPDATE_CALLBACK(Finalize), TT_Async, SRA_UPDATE_CALLBACK(CancelIO));
}

void FTexture2DStreamIn_IO_AsyncReallocate::Finalize(const FContext& Context)
{
	DECLARE_SCOPE_CYCLE_COUNTER(TEXT("FTexture2DStreamIn_IO_AsyncReallocate::Finalize"), STAT_Texture2DStreamInIOAsyncReallocate_Finalize, STATGROUP_StreamingDetails);
	check(Context.CurrentThread == TT_Render);

	FRHICommandListImmediate& RHICmdList = FRHICommandListImmediate::Get();

	ClearIORequests(Context);
	DoUnlockNewMips(RHICmdList, Context);
	DoFinishUpdate(RHICmdList, Context);
}

// ****************************
// ******* Cancel Steps *******
// ****************************

void FTexture2DStreamIn_IO_AsyncReallocate::CancelIO(const FContext& Context)
{
	DECLARE_SCOPE_CYCLE_COUNTER(TEXT("FTexture2DStreamIn_IO_AsyncReallocate::CancelIO"), STAT_Texture2DStreamInIOAsyncReallocate_CancelIO, STATGROUP_StreamingDetails);
	check(Context.CurrentThread == TT_Async);

	ClearIORequests(Context);

	PushTask(Context, TT_None, nullptr, TT_Render, SRA_UPDATE_CALLBACK(Cancel));
}

void FTexture2DStreamIn_IO_AsyncReallocate::Cancel(const FContext& Context)
{
	DECLARE_SCOPE_CYCLE_COUNTER(TEXT("FTexture2DStreamIn_IO_AsyncReallocate::Cancel"), STAT_Texture2DStreamInIOAsyncReallocate_Cancel, STATGROUP_StreamingDetails);
	check(Context.CurrentThread == TT_Render);

	FRHICommandListImmediate& RHICmdList = FRHICommandListImmediate::Get();

	DoUnlockNewMips(RHICmdList, Context);
	DoFinishUpdate(RHICmdList, Context);
	ReportIOError(Context);
}
