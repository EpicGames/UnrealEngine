// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
Texture2DStreamIn_IO_AsyncReallocate.h: Default path for streaming in texture 2D mips.
=============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "Texture2DStreamIn_IO.h"

class FTexture2DStreamIn_IO_AsyncReallocate : public FTexture2DStreamIn_IO
{
public:

	FTexture2DStreamIn_IO_AsyncReallocate(UTexture2D* InTexture, bool InPrioritizedIORequest);

protected:

	// ****************************
	// ******* Update Steps *******
	// ****************************

	// Create an intermediate bigger copy of the texture. (RenderThread)
	void AsyncReallocate(const FContext& Context);
	// Lock each new mips of the intermediate texture. (RenderThread)
	void LockMips(const FContext& Context);
	// Create load requests into each locked mips. (AsyncThread)
	void LoadMips(const FContext& Context);
	// Unlock the mips, apply the intermediate texture and cleaup. (RenderThread)
	void Finalize(const FContext& Context);

	// ****************************
	// ******* Cancel Steps *******
	// ****************************

	// Cancel pending IO then cancel (Async)
	void CancelIO(const FContext& Context);
	// Unlock the mips, cancel load requests and clean up. (RenderThread)
	void Cancel(const FContext& Context);
};
