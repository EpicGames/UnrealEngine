// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
Texture2DStreamIn_IO_AsyncCreate.cpp: Async create path for streaming in texture 2D mips.
=============================================================================*/

#include "Streaming/Texture2DStreamIn_IO_Virtual.h"
#include "Streaming/Texture2DStreamIn_IO.h"
#include "Streaming/Texture2DUpdate.h"

FTexture2DStreamIn_IO_Virtual::FTexture2DStreamIn_IO_Virtual(UTexture2D* InTexture, bool InPrioritizedIORequest) 
	: FTexture2DStreamIn_IO(InTexture, InPrioritizedIORequest)
{
	PushTask(FContext(InTexture, TT_None), TT_Render, SRA_UPDATE_CALLBACK(LockMips), TT_None, nullptr);
}

void FTexture2DStreamIn_IO_Virtual::LockMips(const FContext& Context)
{
	DECLARE_SCOPE_CYCLE_COUNTER(TEXT("FTexture2DStreamIn_IO_Virtual::LockMips"), STAT_Texture2DStreamInIOVirtual_LockMips, STATGROUP_StreamingDetails);
	check(Context.CurrentThread == TT_Render);

	FRHICommandListImmediate& RHICmdList = FRHICommandListImmediate::Get();

	DoConvertToVirtualWithNewMips(RHICmdList, Context);
	DoLockNewMips(RHICmdList, Context);

	PushTask(Context, TT_Async, SRA_UPDATE_CALLBACK(LoadMips), TT_Render, SRA_UPDATE_CALLBACK(Cancel));
}

void FTexture2DStreamIn_IO_Virtual::LoadMips(const FContext& Context)
{
	DECLARE_SCOPE_CYCLE_COUNTER(TEXT("FTexture2DStreamIn_IO_Virtual::LoadMips"), STAT_Texture2DStreamInIOVirtual_LoadMips, STATGROUP_StreamingDetails);
	check(Context.CurrentThread == TT_Async);

	SetIORequests(Context);

	PushTask(Context, TT_Async, SRA_UPDATE_CALLBACK(PostLoadMips), TT_Async, SRA_UPDATE_CALLBACK(CancelIO));
}

void FTexture2DStreamIn_IO_Virtual::PostLoadMips(const FContext& Context)
{
	DECLARE_SCOPE_CYCLE_COUNTER(TEXT("FTexture2DStreamIn_IO_Virtual::PostLoadMips"), STAT_Texture2DStreamInIOVirtual_LoadMips, STATGROUP_StreamingDetails);
	check(Context.CurrentThread == TT_Async);

	ClearIORequests(Context);

	PushTask(Context, TT_Render, SRA_UPDATE_CALLBACK(Finalize), TT_Render, SRA_UPDATE_CALLBACK(Cancel));
}


void FTexture2DStreamIn_IO_Virtual::Finalize(const FContext& Context)
{
	DECLARE_SCOPE_CYCLE_COUNTER(TEXT("FTexture2DStreamIn_IO_Virtual::Finalize"), STAT_Texture2DStreamInIOVirtual_Finalize, STATGROUP_StreamingDetails);
	check(Context.CurrentThread == TT_Render);

	FRHICommandListImmediate& RHICmdList = FRHICommandListImmediate::Get();

	DoUnlockNewMips(RHICmdList, Context);
	if (IntermediateTextureRHI)
	{
		RHICmdList.VirtualTextureSetFirstMipVisible(IntermediateTextureRHI, PendingFirstLODIdx);
	}
	DoFinishUpdate(RHICmdList, Context);
}

// ****************************
// ******* Cancel Steps *******
// ****************************

void FTexture2DStreamIn_IO_Virtual::CancelIO(const FContext& Context)
{
	DECLARE_SCOPE_CYCLE_COUNTER(TEXT("FTexture2DStreamIn_IO_Virtual::CancelIO"), STAT_Texture2DStreamInIOVirtual_CancelIO, STATGROUP_StreamingDetails);
	check(Context.CurrentThread == TT_Async);

	ClearIORequests(Context);

	PushTask(Context, TT_None, nullptr, TT_Render, SRA_UPDATE_CALLBACK(Cancel));
}

void FTexture2DStreamIn_IO_Virtual::Cancel(const FContext& Context)
{
	DECLARE_SCOPE_CYCLE_COUNTER(TEXT("FTexture2DStreamIn_IO_Virtual::Cancel"), STAT_Texture2DStreamInIOVirtual_Cancel, STATGROUP_StreamingDetails);
	check(Context.CurrentThread == TT_Render);

	FRHICommandListImmediate& RHICmdList = FRHICommandListImmediate::Get();

	DoUnlockNewMips(RHICmdList, Context);
	if (IntermediateTextureRHI)
	{
		RHICmdList.VirtualTextureSetFirstMipInMemory(IntermediateTextureRHI, CurrentFirstLODIdx);
	}
	DoFinishUpdate(RHICmdList, Context);
	ReportIOError(Context);
}
