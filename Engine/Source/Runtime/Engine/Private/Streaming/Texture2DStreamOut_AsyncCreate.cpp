// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
Texture2DStreamOut_AsyncCreate.cpp: Definitions of classes used for texture.
=============================================================================*/

#include "Streaming/Texture2DStreamOut_AsyncCreate.h"
#include "Rendering/Texture2DResource.h"
#include "Streaming/Texture2DUpdate.h"


// ****************************
// ******* Update Steps *******
// ****************************

FTexture2DStreamOut_AsyncCreate::FTexture2DStreamOut_AsyncCreate(UTexture2D* InTexture)
	: FTexture2DUpdate(InTexture)
{
	if (!ensure(ResourceState.NumRequestedLODs < ResourceState.NumResidentLODs))
	{
		bIsCancelled = true;
	}

	PushTask(FContext(InTexture, TT_None), TT_Async, SRA_UPDATE_CALLBACK(AsyncCreate), TT_None, nullptr);
}

void FTexture2DStreamOut_AsyncCreate::AsyncCreate(const FContext& Context)
{
	DECLARE_SCOPE_CYCLE_COUNTER(TEXT("FTexture2DStreamOut_AsyncReallocate::AsyncReallocate"), STAT_Texture2DStreamOutAsyncCreate_AsyncReallocate, STATGROUP_StreamingDetails);
	check(Context.CurrentThread == TT_Async);

	if (!IsCancelled() && Context.Resource && Context.Texture)
	{
		const FTexture2DMipMap& RequestedMipMap = *Context.MipsView[PendingFirstLODIdx];

		ensure(IntermediateTextureRHI == nullptr);
		FGraphEventRef CompletionEvent;
		IntermediateTextureRHI = RHIAsyncCreateTexture2D(
			RequestedMipMap.SizeX,
			RequestedMipMap.SizeY,
			Context.Resource->GetPixelFormat(),
			ResourceState.NumRequestedLODs,
			Context.Resource->GetCreationFlags(),
			ERHIAccess::Unknown,
			nullptr,
			0,
			*Context.Resource->GetTextureName().ToString(),
			CompletionEvent);

		if (CompletionEvent)
		{
			TaskSynchronization.Increment();
			FFunctionGraphTask::CreateAndDispatchWhenReady(
				[this]()
				{
					TaskSynchronization.Decrement();
				},
				TStatId{},
				CompletionEvent);
		}
	}

	PushTask(Context, TT_Render, SRA_UPDATE_CALLBACK(Finalize), TT_Render, SRA_UPDATE_CALLBACK(Cancel));
}

void FTexture2DStreamOut_AsyncCreate::Finalize(const FContext& Context)
{
	DECLARE_SCOPE_CYCLE_COUNTER(TEXT("FTexture2DStreamOut_AsyncReallocate::Finalize"), STAT_Texture2DStreamOutAsyncCreate_Finalize, STATGROUP_StreamingDetails);
	check(Context.CurrentThread == TT_Render);

	FRHICommandListImmediate& RHICmdList = FRHICommandListImmediate::Get();

	if (!IsCancelled() && IntermediateTextureRHI && Context.Resource)
	{
		UE::RHI::CopySharedMips_AssumeSRVMaskState(
			RHICmdList,
			Context.Resource->GetTexture2DRHI(),
			IntermediateTextureRHI);
	}

	DoFinishUpdate(RHICmdList, Context);
}

// ****************************
// ******* Cancel Steps *******
// ****************************


void FTexture2DStreamOut_AsyncCreate::Cancel(const FContext& Context)
{
	DECLARE_SCOPE_CYCLE_COUNTER(TEXT("FTexture2DStreamOut_AsyncCreate::Cancel"), STAT_Texture2DStreamOutAsyncCreate_Cancel, STATGROUP_StreamingDetails);
	check(Context.CurrentThread == TT_Render);

	FRHICommandListImmediate& RHICmdList = FRHICommandListImmediate::Get();

	DoFinishUpdate(RHICmdList, Context);
}
