// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
Texture2DStreamOut_AsyncReallocate.cpp: Definitions of classes used for texture.
=============================================================================*/

#include "Streaming/Texture2DStreamOut_AsyncReallocate.h"
#include "Streaming/Texture2DUpdate.h"

// ****************************
// ******* Update Steps *******
// ****************************

FTexture2DStreamOut_AsyncReallocate::FTexture2DStreamOut_AsyncReallocate(UTexture2D* InTexture)
	: FTexture2DUpdate(InTexture) 
{
	if (!ensure(ResourceState.NumRequestedLODs < ResourceState.NumResidentLODs))
	{
		bIsCancelled = true;
	}

	PushTask(FContext(InTexture, TT_None), TT_Render, SRA_UPDATE_CALLBACK(AsyncReallocate), TT_None, nullptr);
}

void FTexture2DStreamOut_AsyncReallocate::AsyncReallocate(const FContext& Context)
{
	DECLARE_SCOPE_CYCLE_COUNTER(TEXT("FTexture2DStreamOut_AsyncReallocate::AsyncReallocate"), STAT_Texture2DStreamOutAsyncReallocate_AsyncReallocate, STATGROUP_StreamingDetails);
	check(Context.CurrentThread == TT_Render);

	FRHICommandListImmediate& RHICmdList = FRHICommandListImmediate::Get();

	DoAsyncReallocate(RHICmdList, Context);
	
	PushTask(Context, TT_Render, SRA_UPDATE_CALLBACK(Finalize), TT_Render, SRA_UPDATE_CALLBACK(Cancel));
}

void FTexture2DStreamOut_AsyncReallocate::Finalize(const FContext& Context)
{
	DECLARE_SCOPE_CYCLE_COUNTER(TEXT("FTexture2DStreamOut_AsyncReallocate::Finalize"), STAT_Texture2DStreamOutAsyncReallocate_Finalize, STATGROUP_StreamingDetails);
	check(Context.CurrentThread == TT_Render);

	FRHICommandListImmediate& RHICmdList = FRHICommandListImmediate::Get();

	DoFinishUpdate(RHICmdList, Context);
}

// ****************************
// ******* Cancel Steps *******
// ****************************


void FTexture2DStreamOut_AsyncReallocate::Cancel(const FContext& Context)
{
	DECLARE_SCOPE_CYCLE_COUNTER(TEXT("FTexture2DStreamOut_AsyncReallocate::Cancel"), STAT_Texture2DStreamOutAsyncReallocate_Cancel, STATGROUP_StreamingDetails);
	check(Context.CurrentThread == TT_Render);

	FRHICommandListImmediate& RHICmdList = FRHICommandListImmediate::Get();

	DoFinishUpdate(RHICmdList, Context);
}
