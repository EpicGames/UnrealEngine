// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
Texture2DStreamOut_Virtual.cpp: Definitions of classes used for texture.
=============================================================================*/

#include "Streaming/Texture2DStreamOut_Virtual.h"
#include "Rendering/Texture2DResource.h"
#include "Streaming/Texture2DUpdate.h"

FTexture2DStreamOut_Virtual::FTexture2DStreamOut_Virtual(UTexture2D* InTexture)
	: FTexture2DUpdate(InTexture) 
{
	if (!ensure(ResourceState.NumRequestedLODs <ResourceState.NumResidentLODs))
	{
		bIsCancelled = true;
	}
	
	PushTask(FContext(InTexture, TT_None), TT_Render, SRA_UPDATE_CALLBACK(Finalize), TT_None, nullptr);
}

// ****************************
// ******* Update Steps *******
// ****************************

void FTexture2DStreamOut_Virtual::Finalize(const FContext& Context)
{
	DECLARE_SCOPE_CYCLE_COUNTER(TEXT("FTexture2DStreamOut_Virtual::Finalize"), STAT_Texture2DStreamOutVirtual_Finalize, STATGROUP_StreamingDetails);
	check(Context.CurrentThread == TT_Render);

	FRHICommandListImmediate& RHICmdList = FRHICommandListImmediate::Get();

	static TConsoleVariableData<int32>* CVarReducedMode = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.PRT.ReducedMemory"));
	check(CVarReducedMode);

	if (!CVarReducedMode->GetValueOnRenderThread() || ResourceState.NumRequestedLODs > ResourceState.NumNonStreamingLODs)
	{
		IntermediateTextureRHI = Context.Resource->GetTexture2DRHI();
		RHICmdList.VirtualTextureSetFirstMipVisible(IntermediateTextureRHI, PendingFirstLODIdx);
		RHICmdList.VirtualTextureSetFirstMipInMemory(IntermediateTextureRHI, PendingFirstLODIdx);
	}
	else
	{
		 DoConvertToNonVirtual(RHICmdList, Context);
	}
	DoFinishUpdate(RHICmdList, Context);
}

// ****************************
// ******* Cancel Steps *******
// ****************************

void FTexture2DStreamOut_Virtual::Cancel(const FContext& Context)
{
	DECLARE_SCOPE_CYCLE_COUNTER(TEXT("FTexture2DStreamOut_Virtual::Cancel"), STAT_Texture2DStreamOutVirtual_Cancel, STATGROUP_StreamingDetails);
	check(Context.CurrentThread == TT_Render);

	FRHICommandListImmediate& RHICmdList = FRHICommandListImmediate::Get();

	DoFinishUpdate(RHICmdList, Context);
}
