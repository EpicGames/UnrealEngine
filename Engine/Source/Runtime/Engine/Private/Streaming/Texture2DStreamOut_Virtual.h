// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
Texture2DStreamOut_Virtual.h: Stream out logic for texture 2D.
=============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "Texture2DUpdate.h"

class FTexture2DStreamOut_Virtual : public FTexture2DUpdate
{
public:

	FTexture2DStreamOut_Virtual(UTexture2D* InTexture);

protected:

	// ****************************
	// ******* Update Steps *******
	// ****************************

	// Reduce visible and resident mips and cleanup. (RenderThread)
	void Finalize(const FContext& Context);

	// ****************************
	// ******* Cancel Steps *******
	// ****************************

	// Cancel the update. (RenderThread)
	void Cancel(const FContext& Context);
};
