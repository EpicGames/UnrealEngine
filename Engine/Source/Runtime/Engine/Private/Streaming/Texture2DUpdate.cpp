// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
Texture2DUpdate.cpp: Helpers to stream in and out mips.
=============================================================================*/

#include "Streaming/Texture2DUpdate.h"
#include "Containers/ResourceArray.h"
#include "Rendering/Texture2DResource.h"
#include "Streaming/RenderAssetUpdate.inl"

// Instantiate TRenderAssetUpdate for FTexture2DUpdateContext
template class TRenderAssetUpdate<FTexture2DUpdateContext>;

#if STATS
extern volatile int64 GPending2DUpdateCount;
volatile int64 GPending2DUpdateCount = 0;
#endif

FTexture2DUpdateContext::FTexture2DUpdateContext(const UTexture2D* InTexture, EThreadType InCurrentThread)
	: Texture(InTexture)
	, CurrentThread(InCurrentThread)
{
	if (InTexture)
	{
		checkSlow(InCurrentThread != FTexture2DUpdate::TT_Render || IsInRenderingThread());
		Resource = Texture->GetResource() ? const_cast<UTexture2D*>(Texture)->GetResource()->GetTexture2DResource() : nullptr;
		if (Resource)
		{
			MipsView = Resource->GetPlatformMipsView();
		}
	}
}

FTexture2DUpdateContext::FTexture2DUpdateContext(const UStreamableRenderAsset* InTexture, EThreadType InCurrentThread)
	: FTexture2DUpdateContext(Cast<UTexture2D>(InTexture), InCurrentThread)
{}

FTexture2DUpdate::FTexture2DUpdate(UTexture2D* InTexture) 
	: TRenderAssetUpdate<FTexture2DUpdateContext>(InTexture)
{
	if (!InTexture || !InTexture->GetResource())
	{
		bIsCancelled = true;
	}

	STAT(FPlatformAtomics::InterlockedIncrement(&GPending2DUpdateCount));
}

FTexture2DUpdate::~FTexture2DUpdate()
{
	ensure(!IntermediateTextureRHI);

	STAT(FPlatformAtomics::InterlockedDecrement(&GPending2DUpdateCount));
}


// ****************************
// ********* Helpers **********
// ****************************

void FTexture2DUpdate::DoAsyncReallocate(FRHICommandListImmediate& RHICmdList, const FContext& Context)
{
	check(Context.CurrentThread == TT_Render);

	if (!IsCancelled() && Context.Texture && Context.Resource)
	{
		const FTexture2DMipMap& RequestedMipMap = *Context.MipsView[PendingFirstLODIdx];

		TaskSynchronization.Set(1);

		ensure(!IntermediateTextureRHI);

		IntermediateTextureRHI = RHICmdList.AsyncReallocateTexture2D(
			Context.Resource->GetTexture2DRHI(),
			ResourceState.NumRequestedLODs,
			RequestedMipMap.SizeX,
			RequestedMipMap.SizeY,
			&TaskSynchronization);
	}
}


//  Transform the texture into a virtual texture.
void FTexture2DUpdate::DoConvertToVirtualWithNewMips(FRHICommandListImmediate& RHICmdList, const FContext& Context)
{
	check(Context.CurrentThread == TT_Render);

	if (!IsCancelled() && Context.Resource)
	{
		// If the texture is not virtual, then make it virtual immediately.
		if (!Context.Resource->IsTextureRHIPartiallyResident())
		{
			const FTexture2DMipMap& MipMap0 = *Context.MipsView[0];

			ensure(!IntermediateTextureRHI);

			// Create a copy of the texture that is a virtual texture.
			FRHITextureCreateDesc Desc =
				FRHITextureCreateDesc::Create2D(TEXT("FTexture2DUpdate"), MipMap0.SizeX, MipMap0.SizeY, Context.Resource->GetPixelFormat())
				.SetNumMips(ResourceState.MaxNumLODs)
				.SetFlags(Context.Resource->GetCreationFlags() | ETextureCreateFlags::Virtual);

			if (Context.Resource->ResourceMem)
			{
				Desc.SetInitActionBulkData(Context.Resource->ResourceMem);
			}

			Desc.DetermineInititialState();

			IntermediateTextureRHI = RHICmdList.CreateTexture(Desc);

			RHICmdList.VirtualTextureSetFirstMipInMemory(IntermediateTextureRHI, CurrentFirstLODIdx);
			RHICmdList.VirtualTextureSetFirstMipVisible(IntermediateTextureRHI, CurrentFirstLODIdx);

			UE::RHI::CopySharedMips_AssumeSRVMaskState(
				RHICmdList,
				Context.Resource->GetTexture2DRHI(),
				IntermediateTextureRHI);
		}
		else
		{
			// Otherwise the current texture is already virtual and we can update it directly.
			IntermediateTextureRHI = Context.Resource->GetTexture2DRHI();
		}
		RHICmdList.VirtualTextureSetFirstMipInMemory(IntermediateTextureRHI, PendingFirstLODIdx);
	}
}

bool FTexture2DUpdate::DoConvertToNonVirtual(FRHICommandListImmediate& RHICmdList, const FContext& Context)
{
	check(Context.CurrentThread == TT_Render);

	// If the texture is virtual, then create a new copy of the texture.
	if (!IsCancelled() && !IntermediateTextureRHI && Context.Texture && Context.Resource)
	{
		if (Context.Resource->IsTextureRHIPartiallyResident())
		{
			const FTexture2DMipMap& PendingFirstMipMap = *Context.MipsView[PendingFirstLODIdx];

			ensure(!IntermediateTextureRHI);

			FRHITextureCreateDesc Desc =
				FRHITextureCreateDesc::Create2D(TEXT("FTexture2DUpdate"), PendingFirstMipMap.SizeX, PendingFirstMipMap.SizeY, Context.Resource->GetPixelFormat())
				.SetNumMips(ResourceState.NumRequestedLODs)
				.SetFlags(Context.Resource->GetCreationFlags());

			if (Context.Resource->ResourceMem)
			{
				Desc.SetInitActionBulkData(Context.Resource->ResourceMem);
			}

			Desc.DetermineInititialState();

			IntermediateTextureRHI = RHICmdList.CreateTexture(Desc);

			UE::RHI::CopySharedMips_AssumeSRVMaskState(
				RHICmdList,
				Context.Resource->GetTexture2DRHI(),
				IntermediateTextureRHI);

			return true;
		}
	}
	return false;
}

void FTexture2DUpdate::DoFinishUpdate(FRHICommandListImmediate& RHICmdList, const FContext& Context)
{
	check(Context.CurrentThread == TT_Render);

	if (IntermediateTextureRHI && Context.Resource)
	{
		if (!IsCancelled())
		{
			Context.Resource->FinalizeStreaming(RHICmdList, IntermediateTextureRHI);
			MarkAsSuccessfullyFinished();
		}
		IntermediateTextureRHI.SafeRelease();

	}
}
