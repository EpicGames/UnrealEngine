// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
Texture2DUpdate.h: Helpers to stream in and out mips.
=============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "RenderAssetUpdate.h"
#include "Engine/Texture2D.h"
#include "Rendering/Texture2DResource.h"
#include "Async/AsyncFileHandle.h"

extern TAutoConsoleVariable<int32> CVarFlushRHIThreadOnSTreamingTextureLocks;

/**
* A context used to update or proceed with the next update step.
* The texture and resource references could be stored in the update object
* but are currently kept outside to avoid lifetime management within the object.
*/
struct FTexture2DUpdateContext
{
	typedef int32 EThreadType;

	FTexture2DUpdateContext(const UTexture2D* InTexture, EThreadType InCurrentThread);

	FTexture2DUpdateContext(const UStreamableRenderAsset* InTexture, EThreadType InCurrentThread);

	EThreadType GetCurrentThread() const
	{
		return CurrentThread;
	}

	/** The texture to update, this must be the same one as the one used when creating the FTexture2DUpdate object. */
	const UTexture2D* Texture = nullptr;
	/** The current 2D resource of this texture. */
	FTexture2DResource* Resource = nullptr;
	/** The array view of streamable mips from the asset. Takes into account FStreamableRenderResourceState::AssetLODBias and FStreamableRenderResourceState::MaxNumLODs. */
	TArrayView<const FTexture2DMipMap*> MipsView;
	/** The thread on which the context was created. */
	EThreadType CurrentThread = 0;
};

// Declare that TRenderAssetUpdate is instantiated for FTexture2DUpdateContext
extern template class TRenderAssetUpdate<FTexture2DUpdateContext>;

/**
 * This class provides a framework for loading and unloading the mips of 2D textures.
 * Each thread essentially calls Tick() until the job is done.
 * The object can be safely deleted when IsCompleted() returns true.
 */
class FTexture2DUpdate : public TRenderAssetUpdate<FTexture2DUpdateContext>
{
public:
	FTexture2DUpdate(UTexture2D* InTexture);

protected:

	virtual ~FTexture2DUpdate();

	// ****************************
	// ********* Helpers **********
	// ****************************

	/** Async Reallocate the texture to the requested size. */
	void DoAsyncReallocate(FRHICommandListImmediate& RHICmdList, const FContext& Context);
	/** Transform the texture into a virtual texture. The virtual texture. */
	void DoConvertToVirtualWithNewMips(FRHICommandListImmediate& RHICmdList, const FContext& Context);
	/** Transform the texture into a non virtual texture. The new texture will have the size of the requested mips to avoid later reallocs. */
	bool DoConvertToNonVirtual(FRHICommandListImmediate& RHICmdList, const FContext& Context);
	/** Apply the new texture (if not cancelled) and finish the update process. When cancelled, the intermediate texture simply gets discarded. */
	void DoFinishUpdate(FRHICommandListImmediate& RHICmdList, const FContext& Context);

	/** The intermediate texture created in the update process. In the virtual path, this can exceptionally end update being the same as the original texture. */
	FTextureRHIRef IntermediateTextureRHI;
};
