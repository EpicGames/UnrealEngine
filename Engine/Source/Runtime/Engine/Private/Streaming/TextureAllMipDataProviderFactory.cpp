// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	TextureAllMipDataProviderFactory.cpp: Implementation of TextureAllMipDataProviderFactory.
=============================================================================*/

#include "Engine/TextureAllMipDataProviderFactory.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(TextureAllMipDataProviderFactory)

UTextureAllMipDataProviderFactory::UTextureAllMipDataProviderFactory(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

