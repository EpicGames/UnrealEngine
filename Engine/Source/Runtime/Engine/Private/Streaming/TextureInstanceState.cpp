// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	TextureInstanceState.cpp: Implementation of content streaming classes.
=============================================================================*/

#include "Streaming/TextureInstanceState.h"
#include "Components/PrimitiveComponent.h"
#include "Engine/Texture.h"
#include "Streaming/TextureInstanceView.inl"
#include "Components/PrimitiveComponent.h"
#include "Streaming/TextureInstanceView.h"
#include "Streaming/StreamingManagerTexture.h"
#include "Async/ParallelFor.h"

FRenderAssetInstanceState::FRenderAssetInstanceState(bool bForDynamicInstances)
	: bIsDynamicInstanceState(bForDynamicInstances)
{
}

int32 FRenderAssetInstanceState::AddBounds(const IPrimitiveComponent* Component)
{
	checkf(bIsDynamicInstanceState, TEXT("This version of AddBounds should only be called by the dynamic instance manager."));
	FBoxSphereBounds BoundsCopy = Component->GetBounds();
	BoundsCopy.SphereRadius = Component->GetStreamingScale();
	return AddBounds(BoundsCopy, PackedRelativeBox_Identity, Component, Component->GetLastRenderTimeOnScreen(), BoundsCopy.Origin, 0, 0, FLT_MAX);
}

int32 FRenderAssetInstanceState::AddBounds(const FBoxSphereBounds& Bounds, uint32 PackedRelativeBox, const IPrimitiveComponent* InComponent, float LastRenderTime, const FVector4& RangeOrigin, float MinDistanceSq, float MinRangeSq, float MaxRangeSq)
{
	check(InComponent);

	int BoundsIndex = INDEX_NONE;

	while (!Bounds4Components.IsValidIndex(BoundsIndex) && FreeBoundIndices.Num() > 0)
	{
		BoundsIndex = FreeBoundIndices.Pop();
	}

	if (!Bounds4Components.IsValidIndex(BoundsIndex))
	{
		BoundsIndex = Bounds4.Num() * 4;
		Bounds4.Push(FBounds4());

		Bounds4Components.Push(nullptr);
		Bounds4Components.Push(nullptr);
		Bounds4Components.Push(nullptr);
		Bounds4Components.Push(nullptr);

		// Since each element contains 4 entries, add the 3 unused ones
		FreeBoundIndices.Push(BoundsIndex + 3);
		FreeBoundIndices.Push(BoundsIndex + 2);
		FreeBoundIndices.Push(BoundsIndex + 1);
	}

	Bounds4[BoundsIndex / 4].Set(BoundsIndex % 4, Bounds, PackedRelativeBox, LastRenderTime, RangeOrigin, MinDistanceSq, MinRangeSq, MaxRangeSq);
	Bounds4Components[BoundsIndex] = InComponent;

	return BoundsIndex;
}

void FRenderAssetInstanceState::RemoveBounds(int32 BoundsIndex)
{
	checkSlow(!FreeBoundIndices.Contains(BoundsIndex));

	// If the BoundsIndex is out of range, the next code will crash.	
	if (!ensure(Bounds4Components.IsValidIndex(BoundsIndex)))
	{
		return;
	}

	// Because components can be removed in CheckRegistrationAndUnpackBounds, which iterates on BoundsToUnpack,
	// here we invalidate the index, instead of removing it, to avoid resizing the array.
	int32 BoundsToUnpackIndex = INDEX_NONE;
	if (BoundsToUnpack.Find(BoundsIndex, BoundsToUnpackIndex))
	{
		BoundsToUnpack[BoundsToUnpackIndex] = INDEX_NONE;
	}

	// If the BoundsIndex is out of range, the next code will crash.	
	if (!ensure(Bounds4Components.IsValidIndex(BoundsIndex)))
	{
		return;
	}

	// If not all indices were freed
	if (1 + FreeBoundIndices.Num() != Bounds4.Num() * 4)
	{
		FreeBoundIndices.Push(BoundsIndex);
		Bounds4[BoundsIndex / 4].Clear(BoundsIndex % 4);
		Bounds4Components[BoundsIndex] = nullptr;
	}
	else
	{
		Bounds4.Empty();
		Bounds4Components.Empty();
		FreeBoundIndices.Empty();
	}
}

void FRenderAssetInstanceState::AddElement(const IPrimitiveComponent* InComponent, const UStreamableRenderAsset* InAsset, int InBoundsIndex, float InTexelFactor, bool InForceLoad, int32*& ComponentLink)
{
	check(InComponent && InAsset);

	// Keep Max texel factor up to date.
	MaxTexelFactor = FMath::Max(InTexelFactor, MaxTexelFactor);

	int32 ElementIndex = INDEX_NONE;
	if (FreeElementIndices.Num())
	{
		ElementIndex = FreeElementIndices.Pop(EAllowShrinking::No);
		check(ElementIndex < Elements.Num());
	}
	else
	{
		ElementIndex = Elements.Num();
		Elements.AddElement(FElement());
	}

	FElement& Element = Elements[ElementIndex];

	Element.Component = InComponent;
	Element.RenderAsset = InAsset;
	Element.BoundsIndex = InBoundsIndex;
	Element.TexelFactor = InTexelFactor;
	Element.bForceLoad = InForceLoad;

	FRenderAssetDesc* AssetDesc = RenderAssetMap.Find(InAsset);
	if (AssetDesc)
	{
		check(AssetDesc->HeadLink < Elements.Num());
		FElement& AssetLinkElement = Elements[AssetDesc->HeadLink];

		// The new inserted element as the head element.
		Element.NextRenderAssetLink = AssetDesc->HeadLink;
		AssetLinkElement.PrevRenderAssetLink = ElementIndex;
		AssetDesc->HeadLink = ElementIndex;
	}
	else
	{
		RenderAssetMap.Add(InAsset, FRenderAssetDesc(ElementIndex, InAsset->GetLODGroupForStreaming()));
		check(Element.NextRenderAssetLink == INDEX_NONE);
	}

	check(Element.PrevRenderAssetLink == INDEX_NONE);

	// Simple sanity check to ensure that the component link passed in param is the right one
	checkSlow(ComponentLink == ComponentMap.Find(InComponent));
	if (ComponentLink)
	{
		// The new inserted element as the head element.
		Element.NextComponentLink = *ComponentLink;
		*ComponentLink = ElementIndex;
	}
	else
	{
		ComponentLink = &ComponentMap.Add(InComponent, ElementIndex);
	}

	// Keep the compiled elements up to date if it was built.
	// This will happen when not all components could be inserted in the incremental build.
	if (HasCompiledElements()) 
	{
		CompiledRenderAssetMap.FindOrAdd(Element.RenderAsset).Add(FCompiledElement(Element));

		if (Element.TexelFactor < 0.f && !InAsset->IsA<UTexture>())
		{
			int32* CountPtr = CompiledNumForcedLODCompMap.Find(Element.RenderAsset);
			if (!CountPtr)
			{
				CompiledNumForcedLODCompMap.Add(Element.RenderAsset, 1);
			}
			else
			{
				++*CountPtr;
			}
		}
	}
}

void FRenderAssetInstanceState::RemoveElement(int32 ElementIndex, int32& NextComponentLink, int32& BoundsIndex, const UStreamableRenderAsset*& Asset)
{
	check(ElementIndex < Elements.Num());
	FElement& Element = Elements[ElementIndex];
	NextComponentLink = Element.NextComponentLink; 
	BoundsIndex = Element.BoundsIndex; 

	// Removed compiled elements. This happens when a static component is not registered after the level became visible.
	if (HasCompiledElements())
	{
		CompiledRenderAssetMap.FindChecked(Element.RenderAsset).RemoveSingleSwap(FCompiledElement(Element), EAllowShrinking::No);

		if (Element.TexelFactor < 0.f
			&& Element.RenderAsset
			&& !Element.RenderAsset->IsA<UTexture>()
			&& !--CompiledNumForcedLODCompMap.FindChecked(Element.RenderAsset))
		{
			CompiledNumForcedLODCompMap.Remove(Element.RenderAsset);
		}
	}

	// Unlink texutres or meshes
	if (Element.RenderAsset)
	{
		if (Element.PrevRenderAssetLink == INDEX_NONE) // If NONE, that means this is the head of the texture/mesh list.
		{
			if (Element.NextRenderAssetLink != INDEX_NONE) // Check if there are other entries for this texture/mesh.
			{
				 // Replace the head
				RenderAssetMap.Find(Element.RenderAsset)->HeadLink = Element.NextRenderAssetLink;
				Elements[Element.NextRenderAssetLink].PrevRenderAssetLink = INDEX_NONE;
			}
			else // Otherwise, remove the texture/mesh entry
			{
				RenderAssetMap.Remove(Element.RenderAsset);
				CompiledRenderAssetMap.Remove(Element.RenderAsset);
				check(!CompiledNumForcedLODCompMap.Find(Element.RenderAsset));
				Asset = Element.RenderAsset;
			}
		}
		else // Otherwise, just relink entries.
		{
			Elements[Element.PrevRenderAssetLink].NextRenderAssetLink = Element.NextRenderAssetLink;

			if (Element.NextRenderAssetLink != INDEX_NONE)
			{
				Elements[Element.NextRenderAssetLink].PrevRenderAssetLink = Element.PrevRenderAssetLink;
			}
		}
	}

	// Clear the element and insert in free list.
	if (1 + FreeElementIndices.Num() != Elements.Num())
	{
		FreeElementIndices.Push(ElementIndex);
		Element = FElement();
	}
	else
	{
		check(RenderAssetMap.IsEmpty());
		Elements.Empty();
		FreeElementIndices.Empty();
	}
}

FORCEINLINE bool operator<(const FBoxSphereBounds& Lhs, const FBoxSphereBounds& Rhs)
{
	// Check that all bites of the structure are used!
	check(sizeof(FBoxSphereBounds) == sizeof(FBoxSphereBounds::Origin) + sizeof(FBoxSphereBounds::BoxExtent) + sizeof(FBoxSphereBounds::SphereRadius));

	return FMemory::Memcmp(&Lhs, &Rhs, sizeof(FBoxSphereBounds)) < 0;
}

void FRenderAssetInstanceState::AddRenderAssetElements(const IPrimitiveComponent* Component, const TArray<FPreAddRenderAssetElement>& RenderAssetElements, int32 BoundsIndex, int32*& ComponentLink)
{
	for (const FPreAddRenderAssetElement& Element : RenderAssetElements)
	{
		AddElement(Component, Element.RenderAsset, BoundsIndex, Element.MergedTexelFactor, Component->IsMipStreamingForced(), ComponentLink);
	}
}

namespace UE::Private
{
static void ForEachRenderAssetTexelFactorGroup(const TArrayView<FStreamingRenderAssetPrimitiveInfo>& InRenderAssetInstanceInfos, TFunctionRef<void(const UStreamableRenderAsset*, float)> Func)
{
	// Loop for each render asset - texel factor group (a group being of same texel factor sign)
	for (int32 InfoIndex = 0; InfoIndex < InRenderAssetInstanceInfos.Num();)
	{
		const FStreamingRenderAssetPrimitiveInfo& Info = InRenderAssetInstanceInfos[InfoIndex];
		float MergedTexelFactor = Info.TexelFactor;
		int32 NumOfMergedElements = 1;

		// Merge all texel factor >= 0 for the same texture, or all those < 0
		if (Info.TexelFactor >= 0)
		{
			for (int32 NextInfoIndex = InfoIndex + 1; NextInfoIndex < InRenderAssetInstanceInfos.Num(); ++NextInfoIndex)
			{
				const FStreamingRenderAssetPrimitiveInfo& NextInfo = InRenderAssetInstanceInfos[NextInfoIndex];
				if (NextInfo.RenderAsset == Info.RenderAsset && NextInfo.TexelFactor >= 0)
				{
					MergedTexelFactor = FMath::Max(MergedTexelFactor, NextInfo.TexelFactor);
					++NumOfMergedElements;
				}
				else
				{
					break;
				}
			}
		}
		else // Info.TexelFactor < 0
		{
			for (int32 NextInfoIndex = InfoIndex + 1; NextInfoIndex < InRenderAssetInstanceInfos.Num(); ++NextInfoIndex)
			{
				const FStreamingRenderAssetPrimitiveInfo& NextInfo = InRenderAssetInstanceInfos[NextInfoIndex];
				if (NextInfo.RenderAsset == Info.RenderAsset && NextInfo.TexelFactor < 0)
				{
					MergedTexelFactor = FMath::Min(MergedTexelFactor, NextInfo.TexelFactor);
					++NumOfMergedElements;
				}
				else
				{
					break;
				}
			}
		}
		Func(Info.RenderAsset, MergedTexelFactor);

		InfoIndex += NumOfMergedElements;
	}
}

static void PreAddRenderAssetElements(const TArrayView<FStreamingRenderAssetPrimitiveInfo>& RenderAssetInstanceInfos, TArray<FRenderAssetInstanceState::FPreAddRenderAssetElement>& OutRenderAssets)
{
	ForEachRenderAssetTexelFactorGroup(RenderAssetInstanceInfos, [&OutRenderAssets](const UStreamableRenderAsset* RenderAsset, float TexelFactor)
	{
		OutRenderAssets.Emplace(RenderAsset, TexelFactor);
	});
}
}

void FRenderAssetInstanceState::AddRenderAssetElements(const IPrimitiveComponent* Component, const TArrayView<FStreamingRenderAssetPrimitiveInfo>& RenderAssetInstanceInfos, int32 BoundsIndex, int32*& ComponentLink)
{
	UE::Private::ForEachRenderAssetTexelFactorGroup(RenderAssetInstanceInfos, [this, Component, BoundsIndex, &ComponentLink](const UStreamableRenderAsset* RenderAsset, float TexelFactor)
	{
		AddElement(Component, RenderAsset, BoundsIndex, TexelFactor, Component->IsMipStreamingForced(), ComponentLink);
	});
}

EAddComponentResult FRenderAssetInstanceState::AddComponents(const TArray<FRenderAssetInstanceState::FPreAddComponentPayload>& Payloads, TFunctionRef<void(const FRenderAssetInstanceState::FPreAddComponentPayload&, EAddComponentResult)> OnAddFailedFunc)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FRenderAssetInstanceState::AddComponents);
	checkf(!bIsDynamicInstanceState, TEXT("Error: trying to add component to dynamic instance manager as static"));
	const int32 PayloadCount = Payloads.Num();
	int32 TotalBoundsCount = 0;
	for (int i = 0; i < PayloadCount; ++i)
	{
		TotalBoundsCount += Payloads[i].Bounds.Num();
	}

	Bounds4.Reserve(Bounds4.Num() + TotalBoundsCount);
	for (int i = 0; i < PayloadCount; ++i)
	{
		EAddComponentResult Result = AddComponentInternal(Payloads[i]);
		if (Result != EAddComponentResult::Success)
		{
			OnAddFailedFunc(Payloads[i], Result);
		}
	}
	return EAddComponentResult::Success;
}

EAddComponentResult FRenderAssetInstanceState::AddComponent(const FPreAddComponentPayload& Payload)
{
	if (Payload.Result == EAddComponentResult::Success)
	{
		Bounds4.Reserve(Bounds4.Num() + Payload.Bounds.Num());
	}
	return AddComponentInternal(Payload);
}

EAddComponentResult FRenderAssetInstanceState::AddComponentInternal(const FPreAddComponentPayload& Payload)
{
	checkf(!bIsDynamicInstanceState, TEXT("Error: trying to add component to dynamic instance manager as static"));
	if (Payload.Result == EAddComponentResult::Success)
	{
		int32* ComponentLink = ComponentMap.Find(Payload.Component);
		const float LastRenderTime = Payload.Component->GetLastRenderTimeOnScreen();
		for (const FPreAddBounds& Bounds : Payload.Bounds)
		{
			const int32 BoundsIndex = AddBounds(Bounds.InfoBounds, Bounds.PackedRelativeBox, Payload.Component, LastRenderTime, Bounds.RangeOrigin, Bounds.MinDistanceSq, Bounds.MinRangeSq, Bounds.MaxRangeSq);
			AddRenderAssetElements(Payload.Component, Bounds.RenderAssets, BoundsIndex, ComponentLink);
			if (Bounds.bIsPacked)
			{
				BoundsToUnpack.Push(BoundsIndex);
			}
		}
	}
	return Payload.Result;
}

void FRenderAssetInstanceState::PreAddComponent(const IPrimitiveComponent* Component, FStreamingTextureLevelContext& LevelContext, float MaxAllowedUIDensity, FPreAddComponentPayload& OutPreAddPayload)
{
	OutPreAddPayload.Result = AddComponentInternal(Component, LevelContext, MaxAllowedUIDensity, nullptr, &OutPreAddPayload);
	check(OutPreAddPayload.Component == Component);
}

EAddComponentResult FRenderAssetInstanceState::AddComponent(const IPrimitiveComponent* Component, FStreamingTextureLevelContext& LevelContext, float MaxAllowedUIDensity)
{
	return AddComponentInternal(Component, LevelContext, MaxAllowedUIDensity, this, nullptr);
}

EAddComponentResult FRenderAssetInstanceState::AddComponentInternal(const IPrimitiveComponent* Component, FStreamingTextureLevelContext& LevelContext, float MaxAllowedUIDensity, FRenderAssetInstanceState* State, FPreAddComponentPayload* OutPreAddPayload)
{
	check(State || OutPreAddPayload);
	checkf(!State || !State->bIsDynamicInstanceState, TEXT("Error: trying to add component to dynamic instance manager as static"));

	if (OutPreAddPayload)
	{
		OutPreAddPayload->Component = Component;
	}

	check(Component);
	TArray<FStreamingRenderAssetPrimitiveInfo> RenderAssetInstanceInfos = LevelContext.GetStreamingRenderAssetInfoWithNULLRemoval(Component);

	const float ComponentScale = Component->GetStreamingScale();
	if (ComponentScale != 1.f)
	{
		for (FStreamingRenderAssetPrimitiveInfo& Info : RenderAssetInstanceInfos)
		{
			Info.TexelFactor *= Info.bAffectedByComponentScale ? ComponentScale : 1.f;
		}
	}

	// Texture entries are guarantied to be relevant here, except for bounds if the component is not registered.
	if (!RenderAssetInstanceInfos.Num())
	{
		return EAddComponentResult::Fail;
	}

	// First check if all entries are below the max allowed UI density, otherwise abort immediately.
	if (MaxAllowedUIDensity > 0)
	{
		for (const FStreamingRenderAssetPrimitiveInfo& Info : RenderAssetInstanceInfos)
		{
			if (Info.TexelFactor > MaxAllowedUIDensity)
			{
				return EAddComponentResult::Fail_UIDensityConstraint;
			}
		}
	}

	if (!Component->IsRegistered())
	{
		// When the components are not registered, the bound will be generated from PackedRelativeBox in CheckRegistrationAndUnpackBounds.
		// Otherwise, the entry is not usable as we don't know the bound to use. The component will need to be reinstered later, once registered.
		// it will not be possible to recreate the bounds correctly.
		for (const FStreamingRenderAssetPrimitiveInfo& Info : RenderAssetInstanceInfos)
		{
			if (!Info.PackedRelativeBox)
			{
				return EAddComponentResult::Fail;
			}
		}

		// Sort by PackedRelativeBox, to identical bounds identical entries together
		// Sort by Texture to merge duplicate texture entries.
		// Then sort by TexelFactor, to merge negative entries together.
		RenderAssetInstanceInfos.Sort([](const FStreamingRenderAssetPrimitiveInfo& Lhs, const FStreamingRenderAssetPrimitiveInfo& Rhs)
		{
			if (Lhs.PackedRelativeBox == Rhs.PackedRelativeBox)
			{
				if (Lhs.RenderAsset == Rhs.RenderAsset)
				{
					return Lhs.TexelFactor < Rhs.TexelFactor;
				}
				return Lhs.RenderAsset < Rhs.RenderAsset;
			}
			return Lhs.PackedRelativeBox < Rhs.PackedRelativeBox;
		});

		int32* ComponentLink = State ? State->ComponentMap.Find(Component) : nullptr;

		if (State)
		{
			State->Bounds4.Reserve(State->Bounds4.Num() + RenderAssetInstanceInfos.Num());
		}

		// Loop for each bound.
		for (int32 InfoIndex = 0; InfoIndex < RenderAssetInstanceInfos.Num();)
		{
			const FStreamingRenderAssetPrimitiveInfo& Info = RenderAssetInstanceInfos[InfoIndex];

			int32 NumOfBoundReferences = 1;
			for (int32 NextInfoIndex = InfoIndex + 1; NextInfoIndex < RenderAssetInstanceInfos.Num() && RenderAssetInstanceInfos[NextInfoIndex].PackedRelativeBox == Info.PackedRelativeBox; ++NextInfoIndex)
			{
				++NumOfBoundReferences;
			}

			if (State)
			{
				const int32 BoundsIndex = State->AddBounds(FBoxSphereBounds(ForceInit), Info.PackedRelativeBox, Component, Component->GetLastRenderTimeOnScreen(), FVector(ForceInit), 0, 0, FLT_MAX);
				State->AddRenderAssetElements(Component, TArrayView<FStreamingRenderAssetPrimitiveInfo>(RenderAssetInstanceInfos.GetData() + InfoIndex, NumOfBoundReferences), BoundsIndex, ComponentLink);
				State->BoundsToUnpack.Push(BoundsIndex);
			}
			else
			{
				check(OutPreAddPayload);
				FPreAddBounds& OutBounds = OutPreAddPayload->Bounds.Emplace_GetRef(FBoxSphereBounds(ForceInit), Info.PackedRelativeBox, true, FVector(ForceInit), 0, 0, FLT_MAX);
				UE::Private::PreAddRenderAssetElements(TArrayView<FStreamingRenderAssetPrimitiveInfo>(RenderAssetInstanceInfos.GetData() + InfoIndex, NumOfBoundReferences), OutBounds.RenderAssets);
			}

			InfoIndex += NumOfBoundReferences;
		}
	}
	else
	{
		// Sort by Bounds, to merge identical bounds entries together
		// Sort by Texture to merge duplicate texture entries.
		// Then sort by TexelFactor, to merge negative entries together.
		RenderAssetInstanceInfos.Sort([](const FStreamingRenderAssetPrimitiveInfo& Lhs, const FStreamingRenderAssetPrimitiveInfo& Rhs)
		{
			if (Lhs.Bounds == Rhs.Bounds)
			{
				if (Lhs.RenderAsset == Rhs.RenderAsset)
				{
					return Lhs.TexelFactor < Rhs.TexelFactor;
				}
				return Lhs.RenderAsset < Rhs.RenderAsset;
			}
			return Lhs.Bounds < Rhs.Bounds;
		});

		int32* ComponentLink = State ? State->ComponentMap.Find(Component) : nullptr;
		float MinDistanceSq = 0, MinRangeSq = 0, MaxRangeSq = FLT_MAX;

		if (State)
		{
			State->Bounds4.Reserve(State->Bounds4.Num() + RenderAssetInstanceInfos.Num());
		}

		// Loop for each bound.
		for (int32 InfoIndex = 0; InfoIndex < RenderAssetInstanceInfos.Num();)
		{
			const FStreamingRenderAssetPrimitiveInfo& Info = RenderAssetInstanceInfos[InfoIndex];

			int32 NumOfBoundReferences = 1;
			for (int32 NextInfoIndex = InfoIndex + 1; NextInfoIndex < RenderAssetInstanceInfos.Num() && RenderAssetInstanceInfos[NextInfoIndex].Bounds == Info.Bounds; ++NextInfoIndex)
			{
				++NumOfBoundReferences;
			}

			GetDistanceAndRange(Component, Info.Bounds, MinDistanceSq, MinRangeSq, MaxRangeSq);
			if (State)
			{
				const int32 BoundsIndex = State->AddBounds(Info.Bounds, PackedRelativeBox_Identity, Component, Component->GetLastRenderTimeOnScreen(), Component->GetBounds().Origin, MinDistanceSq, MinRangeSq, MaxRangeSq);
				State->AddRenderAssetElements(Component, TArrayView<FStreamingRenderAssetPrimitiveInfo>(RenderAssetInstanceInfos.GetData() + InfoIndex, NumOfBoundReferences), BoundsIndex, ComponentLink);
			}
			else
			{
				check(OutPreAddPayload);
				FPreAddBounds& OutBounds = OutPreAddPayload->Bounds.Emplace_GetRef(Info.Bounds, PackedRelativeBox_Identity, false, Component->GetBounds().Origin, MinDistanceSq, MinRangeSq, MaxRangeSq);
				UE::Private::PreAddRenderAssetElements(TArrayView<FStreamingRenderAssetPrimitiveInfo>(RenderAssetInstanceInfos.GetData() + InfoIndex, NumOfBoundReferences), OutBounds.RenderAssets);
			}

			InfoIndex += NumOfBoundReferences;
		}
	}
	return EAddComponentResult::Success;
}

void FRenderAssetInstanceState::PreAddComponentIgnoreBounds(const IPrimitiveComponent* Component, FStreamingTextureLevelContext& LevelContext, FPreAddComponentIgnoreBoundsPayload& OutPayload)
{
	OutPayload.Result = AddComponentIgnoreBoundsInternal(Component, LevelContext, nullptr, &OutPayload);
	check(OutPayload.Component == Component);
}

EAddComponentResult FRenderAssetInstanceState::AddComponentIgnoreBounds(const FPreAddComponentIgnoreBoundsPayload& Payload)
{
	checkf(bIsDynamicInstanceState, TEXT("Error: trying to add component to static instance manager as dynamic"));
	if (Payload.Result == EAddComponentResult::Success)
	{
		int32* ComponentLink = ComponentMap.Find(Payload.Component);
		AddRenderAssetElements(Payload.Component, Payload.RenderAssets, AddBounds(Payload.Component), ComponentLink);
	}
	return Payload.Result;
}

EAddComponentResult FRenderAssetInstanceState::AddComponentIgnoreBounds(const IPrimitiveComponent* Component, FStreamingTextureLevelContext& LevelContext)
{
	return AddComponentIgnoreBoundsInternal(Component, LevelContext, this, nullptr);
}

EAddComponentResult FRenderAssetInstanceState::AddComponentIgnoreBoundsInternal(const IPrimitiveComponent* Component, FStreamingTextureLevelContext& LevelContext, FRenderAssetInstanceState* State, FPreAddComponentIgnoreBoundsPayload* OutPreAddPayload)
{
	check(Component->IsRegistered()); // Must be registered otherwise bounds are invalid.
	check(State || OutPreAddPayload);
	checkf(!State || State->bIsDynamicInstanceState, TEXT("Error: trying to add component to static instance manager as dynamic"));

	if (OutPreAddPayload)
	{
		OutPreAddPayload->Component = Component;
	}

	TArray<FStreamingRenderAssetPrimitiveInfo> RenderAssetInstanceInfos = LevelContext.GetStreamingRenderAssetInfoWithNULLRemoval(Component);

	if (!RenderAssetInstanceInfos.Num())
	{
		return EAddComponentResult::Fail;
	}

	// Sort by Texture to merge duplicate texture entries.
	// Then sort by TexelFactor, to merge negative entries together.
	RenderAssetInstanceInfos.Sort([](const FStreamingRenderAssetPrimitiveInfo& Lhs, const FStreamingRenderAssetPrimitiveInfo& Rhs)
	{
		if (Lhs.RenderAsset == Rhs.RenderAsset)
		{
			return Lhs.TexelFactor < Rhs.TexelFactor;
		}
		return Lhs.RenderAsset < Rhs.RenderAsset;
	});

	int32* ComponentLink = State ? State->ComponentMap.Find(Component) : nullptr;
	if (State)
	{
		State->AddRenderAssetElements(Component, RenderAssetInstanceInfos, State->AddBounds(Component), ComponentLink);
	}
	else
	{
		check(OutPreAddPayload);
		UE::Private::PreAddRenderAssetElements(RenderAssetInstanceInfos, OutPreAddPayload->RenderAssets);
	}
	return EAddComponentResult::Success;
}


void FRenderAssetInstanceState::RemoveComponentByHandle(FRemovedComponentHandle ElementIndex, FRemovedRenderAssetArray* RemovedRenderAssets)
{
	TArray<int32, TInlineAllocator<12>> RemovedBoundsIndices;

	while (ElementIndex != INDEX_NONE)
	{
		int32 BoundsIndex = INDEX_NONE;
		const UStreamableRenderAsset* Asset = nullptr;

		RemoveElement(ElementIndex, ElementIndex, BoundsIndex, Asset);

		if (BoundsIndex != INDEX_NONE)
		{
			RemovedBoundsIndices.AddUnique(BoundsIndex);
		}

		if (Asset && RemovedRenderAssets)
		{
			RemovedRenderAssets->AddUnique(Asset);
		}
	};

	for (int32 Index = 0; Index < RemovedBoundsIndices.Num(); ++Index)
	{
		RemoveBounds(RemovedBoundsIndices[Index]);
	}
}

void FRenderAssetInstanceState::RemoveComponent(const IPrimitiveComponent* Component, FRemovedRenderAssetArray* RemovedRenderAssets)
{
	int32 ElementIndex = INDEX_NONE;
	ComponentMap.RemoveAndCopyValue(Component, ElementIndex);

	RemoveComponentByHandle(ElementIndex, RemovedRenderAssets);
}

bool FRenderAssetInstanceState::RemoveComponentReferences(const IPrimitiveComponent* Component)
{
	// Because the async streaming task could be running, we can't change the async view state. 
	// We limit ourself to clearing the component ptr to avoid invalid access when updating visibility.

	int32* ComponentLink = ComponentMap.Find(Component);
	if (ComponentLink)
	{
		int32 ElementIndex = *ComponentLink;
		if (bIsDynamicInstanceState)
		{
			PendingRemoveComponents.Add(ElementIndex);
		}

		while (ElementIndex != INDEX_NONE)
		{
			FElement& Element = Elements[ElementIndex];
			if (Element.BoundsIndex != INDEX_NONE)
			{
				Bounds4Components[Element.BoundsIndex] = nullptr;
			}
			Element.Component = nullptr;

			ElementIndex = Element.NextComponentLink;
		}

		ComponentMap.Remove(Component);
		return true;
	}
	else
	{
		return false;
	}
}

void FRenderAssetInstanceState::FlushPendingRemoveComponents(FRemovedRenderAssetArray& RemovedRenderAssets)
{
	for (int32 Index = 0; Index < PendingRemoveComponents.Num(); ++Index)
	{
		const FRemovedComponentHandle HeadElementIndex = PendingRemoveComponents[Index];
		RemoveComponentByHandle(HeadElementIndex, &RemovedRenderAssets);
	}
	PendingRemoveComponents.Reset();
}

void FRenderAssetInstanceState::UpdateBounds(const IPrimitiveComponent* Component)
{
	checkf(bIsDynamicInstanceState, TEXT("Bounds shouldn't be updated after creation unless the instances are dynamic"));

	if (int32* ComponentLink = ComponentMap.Find(Component))
	{
		int32 ElementIndex = *ComponentLink;
		while (ElementIndex != INDEX_NONE)
		{
			const FElement& Element = Elements[ElementIndex];
			if (Element.BoundsIndex != INDEX_NONE)
			{
				const FBoxSphereBounds& Bounds = Component->GetBounds();
				Bounds4[Element.BoundsIndex / 4].FullUpdate(Element.BoundsIndex % 4, Bounds.Origin, Bounds.BoxExtent, Component->GetStreamingScale(), Component->GetLastRenderTimeOnScreen());
			}
			ElementIndex = Element.NextComponentLink;
		}
	}
}

bool FRenderAssetInstanceState::UpdateBounds(int32 BoundIndex)
{
	checkf(bIsDynamicInstanceState, TEXT("Bounds shouldn't be updated after creation unless the instances are dynamic"));

	const IPrimitiveComponent* Component = ensure(Bounds4Components.IsValidIndex(BoundIndex)) ? Bounds4Components[BoundIndex] : nullptr;
	if (Component)
	{
		const FBoxSphereBounds& Bounds = Component->GetBounds();
		Bounds4[BoundIndex / 4].FullUpdate(BoundIndex % 4, Bounds.Origin, Bounds.BoxExtent, Component->GetStreamingScale(), Component->GetLastRenderTimeOnScreen());
		return true;
	}
	else
	{
		return false;
	}
}

bool FRenderAssetInstanceState::ConditionalUpdateBounds(int32 BoundIndex)
{
	checkf(bIsDynamicInstanceState, TEXT("Bounds shouldn't be updated after creation unless the instances are dynamic"));

	const IPrimitiveComponent* Component = ensure(Bounds4Components.IsValidIndex(BoundIndex)) ? Bounds4Components[BoundIndex] : nullptr;
	if (Component)
	{
		const FBoxSphereBounds& Bounds = Component->GetBounds();

		if (!Component->IsStaticMobility())
		{
			// Check if the bound is coherent as it could be updated while we read it (from async task).
			// We don't have to check the position, as if it was partially updated, this should be ok (interp)
			static_assert((offsetof(FBoxSphereBounds, BoxExtent) + sizeof(Bounds.BoxExtent)) == (offsetof(FBoxSphereBounds, SphereRadius)), "Memory layout for FBoxSphereBounds has changed");
			enum
			{
				X = 0,
				Y,
				Z,
				R
			};

			VectorRegister4Float XYZRData = MakeVectorRegisterFloatFromDouble(VectorLoad(&Bounds.BoxExtent.X));		//X,Y,Z,Radius
			XYZRData = VectorMultiply(XYZRData, XYZRData);
			AlignedFloat4 XYZRSquared(XYZRData);

			if (0.5f * FMath::Min3<float>(XYZRSquared[X], XYZRSquared[Y], XYZRSquared[Z]) <= XYZRSquared[R] && XYZRSquared[R] <= 2.f * (XYZRSquared[X] + XYZRSquared[Y] + XYZRSquared[Z]))
			{
				Bounds4[BoundIndex / 4].FullUpdate(BoundIndex % 4, Bounds.Origin, Bounds.BoxExtent, Component->GetStreamingScale(), Component->GetLastRenderTimeOnScreen());
				return true;
			}
		}
		else // Otherwise we assume it is guarantied to be good.
		{
			Bounds4[BoundIndex / 4].FullUpdate(BoundIndex % 4, Bounds.Origin, Bounds.BoxExtent, Component->GetStreamingScale(), Component->GetLastRenderTimeOnScreen());
			return true;
		}
	}
	return false;
}


void FRenderAssetInstanceState::UpdateLastRenderTimeAndMaxDrawDistance(int32 BoundIndex)
{
	const IPrimitiveComponent* Component = ensure(Bounds4Components.IsValidIndex(BoundIndex)) ? Bounds4Components[BoundIndex] : nullptr;
	if (Component)
	{
		const int32 Bounds4Idx = BoundIndex / 4;
		const int32 SubIdx = BoundIndex & 3;
		Bounds4[Bounds4Idx].UpdateLastRenderTime(SubIdx, Component->GetLastRenderTimeOnScreen());
		// The min draw distances of HLODs can change dynamically (see Tick, PauseDitherTransition,
		// and StartDitherTransition methods of ALODActor)
		const IPrimitiveComponent* LODParent = Component->GetLODParentPrimitive();
		if (LODParent)
		{
			const float MaxRangeSq = FRenderAssetInstanceView::GetMaxDrawDistSqWithLODParent(
				Component->GetBounds().Origin,
				LODParent->GetBounds().Origin,
				LODParent->GetMinDrawDistance(),
				LODParent->GetBounds().SphereRadius);
			Bounds4[Bounds4Idx].UpdateMaxDrawDistanceSquared(SubIdx, MaxRangeSq);
		}
	}
}

uint32 FRenderAssetInstanceState::GetAllocatedSize() const
{
	int32 CompiledElementsSize = 0;
	for (TMap<const UStreamableRenderAsset*, TArray<FCompiledElement> >::TConstIterator It(CompiledRenderAssetMap); It; ++It)
	{
		CompiledElementsSize += It.Value().GetAllocatedSize();
	}

	return Bounds4.GetAllocatedSize() +
		Bounds4Components.GetAllocatedSize() +
		Elements.GetAllocatedSize() +
		FreeBoundIndices.GetAllocatedSize() +
		FreeElementIndices.GetAllocatedSize() +
		RenderAssetMap.GetAllocatedSize() +
		CompiledRenderAssetMap.GetAllocatedSize() + CompiledElementsSize +
		CompiledNumForcedLODCompMap.GetAllocatedSize() +
		ComponentMap.GetAllocatedSize();
}

int32 FRenderAssetInstanceState::CompileElements()
{
	CompiledRenderAssetMap.Empty();
	CompiledNumForcedLODCompMap.Empty();
	MaxTexelFactor = 0;

	TArray<const UStreamableRenderAsset*> RenderAssets;
	RenderAssets.Reserve(RenderAssetMap.Num());

	// First create an entry for all elements, so that there are no reallocs when inserting each compiled elements.
	for (TMap<const UStreamableRenderAsset*, FRenderAssetDesc>::TConstIterator AssetIt(RenderAssetMap); AssetIt; ++AssetIt)
	{
		const UStreamableRenderAsset* Asset = AssetIt.Key();
		CompiledRenderAssetMap.Add(Asset);
		RenderAssets.Add(Asset);
	}

	struct FCompileElementsContext
	{
		float MaxTexelFactor = 0.0f;
		TArray<TPair<const UStreamableRenderAsset*, int32>> CompiledNumForcedLODCompArray;
	};
	TArray<FCompileElementsContext> Contexts;

	// Then fill in each array.
	int32 MinBatchSize = 1;
	bool bIsParallelForAllowed = FRenderAssetStreamingManager::IsParallelForAllowedDuringIncrementalUpdate(RenderAssets.Num(), MinBatchSize);
	ParallelForWithTaskContext(TEXT("CompileElements"), Contexts, RenderAssets.Num(), MinBatchSize, [this, &RenderAssets](FCompileElementsContext& Context, int32 Index)
	{
		const UStreamableRenderAsset* Asset = RenderAssets[Index];
		const bool bIsNonTexture = Asset && !Asset->IsA<UTexture>();
		TArray<FCompiledElement>& CompiledElemements = CompiledRenderAssetMap.FindChecked(Asset);

		int32 CompiledElementCount = 0;
		for (auto ElementIt = GetElementIterator(Asset); ElementIt; ++ElementIt)
		{
			++CompiledElementCount;
		}
		CompiledElemements.AddUninitialized(CompiledElementCount);

		int32 CompiledNumForcedLODCompMapCount = 0;
		CompiledElementCount = 0;
		for (auto ElementIt = GetElementIterator(Asset); ElementIt; ++ElementIt)
		{
			const float TexelFactor = ElementIt.GetTexelFactor();

			if (bIsNonTexture && TexelFactor < 0.f)
			{
				++CompiledNumForcedLODCompMapCount;
			}

			// No need to care about force load as MaxTexelFactor is used to ignore far away levels.
			Context.MaxTexelFactor = FMath::Max(TexelFactor, Context.MaxTexelFactor);

			CompiledElemements[CompiledElementCount].BoundsIndex = ElementIt.GetBoundsIndex();
			CompiledElemements[CompiledElementCount].TexelFactor = TexelFactor;
			CompiledElemements[CompiledElementCount].bForceLoad = ElementIt.GetForceLoad();

			++CompiledElementCount;
		}

		if (CompiledNumForcedLODCompMapCount > 0)
		{
			Context.CompiledNumForcedLODCompArray.Emplace(Asset, CompiledNumForcedLODCompMapCount);
		}
	}, bIsParallelForAllowed ? EParallelForFlags::None : EParallelForFlags::ForceSingleThread);

	for (const FCompileElementsContext& Context : Contexts)
	{
		MaxTexelFactor = FMath::Max(MaxTexelFactor, Context.MaxTexelFactor);
		for (const TPair<const UStreamableRenderAsset*, int32>& Pair : Context.CompiledNumForcedLODCompArray)
		{
			CompiledNumForcedLODCompMap.Add(Pair.Key, Pair.Value);
		}
	}
	return CompiledRenderAssetMap.Num();
}

int32 FRenderAssetInstanceState::CheckRegistrationAndUnpackBounds(TArray<const IPrimitiveComponent*>& RemovedComponents)
{
	for (int32 BoundIndex : BoundsToUnpack)
	{
		if (Bounds4Components.IsValidIndex(BoundIndex))
		{
			if (const IPrimitiveComponent* Component = Bounds4Components[BoundIndex])
			{
				// At this point the component must be registered. If the proxy is valid, reject any component without one.
				// This would be hidden primitives, and also editor / debug primitives.
				if (Component->IsRegistered() && (!Component->IsRenderStateCreated() || Component->GetSceneProxy()))
				{
					Bounds4[BoundIndex / 4].UnpackBounds(BoundIndex % 4, Component);
				}
				else // Here we can remove the component, as the async task is not yet using this.
				{
					RemoveComponent(Component, nullptr);
					RemovedComponents.Add(Component);
				}
			}
		}
	}

	const int32 NumSteps = BoundsToUnpack.Num();
	BoundsToUnpack.Empty();
	return NumSteps;
}

bool FRenderAssetInstanceState::MoveBound(int32 SrcBoundIndex, int32 DstBoundIndex)
{
	check(!HasCompiledElements() && !BoundsToUnpack.Num()); // Defrag is for the dynamic elements which does not support dynamic compiled elements.

	if (Bounds4Components.IsValidIndex(DstBoundIndex) && Bounds4Components.IsValidIndex(SrcBoundIndex) && !Bounds4Components[DstBoundIndex] && Bounds4Components[SrcBoundIndex])
	{
		int32 FreeListIndex = FreeBoundIndices.Find(DstBoundIndex);
		if (FreeListIndex == INDEX_NONE)
		{
			return false; // The destination is not free.
		}
		// Update the free list.
		FreeBoundIndices[FreeListIndex] = SrcBoundIndex;

		const IPrimitiveComponent* Component = Bounds4Components[SrcBoundIndex];

		// Update the elements.
		if (int32* ComponentLink = ComponentMap.Find(Component))
		{
			int32 ElementIndex = *ComponentLink;
			while (ElementIndex != INDEX_NONE)
			{
				FElement& Element = Elements[ElementIndex];
				
				// Sanity check to ensure elements and bounds are still linked correctly!
				check(Element.Component == Component);

				if (Element.BoundsIndex == SrcBoundIndex)
				{
					Element.BoundsIndex = DstBoundIndex;
				}
				ElementIndex = Element.NextComponentLink;
			}
		}

		// Update the component ptrs.
		Bounds4Components[DstBoundIndex] = Component;
		Bounds4Components[SrcBoundIndex] = nullptr;

		UpdateBounds(DstBoundIndex); // Update the bounds using the component.
		Bounds4[SrcBoundIndex / 4].Clear(SrcBoundIndex % 4);	

		return true;
	}
	else
	{
		return false;
	}
}

void FRenderAssetInstanceState::TrimBounds()
{
	// Cannot trim if there are pending removes. Corresponding Bounds4Components entries are nullptrs but not actually free.
	if (PendingRemoveComponents.Num() > 0)
	{
		return;
	}

	const int32 DefragThreshold = 8; // Must be a multiple of 4
	check(NumBounds4() * 4 == NumBounds());

	bool bUpdateFreeBoundIndices = false;

	// Here we check the bound components from low indices to high indices
	// because there are more chance that the lower range indices fail.
	// (since the incremental update move null component to the end)
	bool bDefragRangeIsFree = true;
	while (bDefragRangeIsFree)
	{
		const int32 LowerBoundThreshold = NumBounds() - DefragThreshold;
		if (Bounds4Components.IsValidIndex(LowerBoundThreshold))
		{
			for (int BoundIndex = LowerBoundThreshold; BoundIndex < NumBounds(); ++BoundIndex)
			{
				if (Bounds4Components[BoundIndex])
				{
					bDefragRangeIsFree = false;
					break;
				}
				else
				{
					checkSlow(FreeBoundIndices.Contains(BoundIndex));
				}
			}

			if (bDefragRangeIsFree)
			{
				Bounds4.RemoveAt(Bounds4.Num() - DefragThreshold / 4, DefragThreshold / 4, EAllowShrinking::No);
				Bounds4Components.RemoveAt(Bounds4Components.Num() - DefragThreshold, DefragThreshold, EAllowShrinking::No);
				bUpdateFreeBoundIndices = true;
			}
		}
		else
		{
			bDefragRangeIsFree = false;
		}
	} 

	if (bUpdateFreeBoundIndices)
	{
		// The bounds are cleared outside the range loop to prevent parsing elements multiple times.
		for (int Index = 0; Index < FreeBoundIndices.Num(); ++Index)
		{
			if (FreeBoundIndices[Index] >= NumBounds())
			{
				FreeBoundIndices.RemoveAtSwap(Index);
				--Index;
			}
		}
		check(NumBounds4() * 4 == NumBounds());
	}
}

void FRenderAssetInstanceState::OffsetBounds(const FVector& Offset)
{
	for (int32 BoundIndex = 0; BoundIndex < Bounds4Components.Num(); ++BoundIndex)
	{
		if (Bounds4Components[BoundIndex])
		{
			Bounds4[BoundIndex / 4].OffsetBounds(BoundIndex % 4, Offset);
		}
	}
}
