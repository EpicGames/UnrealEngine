// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
TextureInstanceState.h: Definitions of classes used for texture streaming.
=============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "TextureInstanceView.h"
#include "Streaming/TextureStreamingHelpers.h"
#include "Containers/ArrayView.h"

class FStreamingTextureLevelContext;
class ULevel;
class UPrimitiveComponent;
struct FStreamingRenderAssetPrimitiveInfo;

enum class EAddComponentResult : uint8
{
	Fail,
	Fail_UIDensityConstraint,
	Success
};

// Can be used either for static primitives or dynamic primitives
class FRenderAssetInstanceState : public FRenderAssetInstanceView
{
public:
	struct FPreAddRenderAssetElement
	{
		FPreAddRenderAssetElement(const UStreamableRenderAsset* InRenderAsset, float InMergedTexelFactor)
			: RenderAsset(InRenderAsset)
			, MergedTexelFactor(InMergedTexelFactor)
		{}

		const UStreamableRenderAsset* RenderAsset;
		float MergedTexelFactor;
	};

	struct FPreAddBounds
	{
		FPreAddBounds(FBoxSphereBounds InInfoBounds, uint32 InPackedRelativeBox, bool bInIsPacked, const FVector& InRangeOrigin, float InMinDistanceSq, float InMinRangeSq, float InMaxRangeSq)
			: InfoBounds(InInfoBounds)
			, PackedRelativeBox(InPackedRelativeBox)
			, RangeOrigin(InRangeOrigin)
			, MinDistanceSq(InMinDistanceSq)
			, MinRangeSq(InMinRangeSq)
			, MaxRangeSq(InMaxRangeSq)
			, bIsPacked(bInIsPacked)
		{}

		FBoxSphereBounds InfoBounds;
		uint32 PackedRelativeBox;
		FVector RangeOrigin;
		float MinDistanceSq;
		float MinRangeSq;
		float MaxRangeSq;
		bool bIsPacked;
		TArray<FPreAddRenderAssetElement> RenderAssets;
	};

	struct FPreAddComponentPayload
	{
		const IPrimitiveComponent* Component;
		EAddComponentResult Result;
		TArray<FPreAddBounds> Bounds;
	};

	struct FPreAddComponentIgnoreBoundsPayload
	{
		const IPrimitiveComponent* Component;
		EAddComponentResult Result;
		TArray<FPreAddRenderAssetElement> RenderAssets;
	};

	FRenderAssetInstanceState(bool bForDynamicInstances);

	static void PreAddComponent(const IPrimitiveComponent* Component, FStreamingTextureLevelContext& LevelContext, float MaxAllowedUIDensity, FPreAddComponentPayload& OutPayload);
	static void PreAddComponentIgnoreBounds(const IPrimitiveComponent* Component, FStreamingTextureLevelContext& LevelContext, FPreAddComponentIgnoreBoundsPayload& OutPayload);

	// Will also remove bounds
	EAddComponentResult AddComponent(const IPrimitiveComponent* Component, FStreamingTextureLevelContext& LevelContext, float MaxAllowedUIDensity);
	EAddComponentResult AddComponent(const FPreAddComponentPayload& Payload);
	EAddComponentResult AddComponents(const TArray<FRenderAssetInstanceState::FPreAddComponentPayload>& Payloads, TFunctionRef<void(const FRenderAssetInstanceState::FPreAddComponentPayload&, EAddComponentResult)> OnAddFailedFunc);

	// Similar to AddComponent, but ignore the streaming data bounds. Used for dynamic components. A faster implementation that does less processing.
	EAddComponentResult AddComponentIgnoreBounds(const IPrimitiveComponent* Component, FStreamingTextureLevelContext& LevelContext);
	EAddComponentResult AddComponentIgnoreBounds(const FPreAddComponentIgnoreBoundsPayload& Payload);

	FORCEINLINE bool HasComponentReferences(const IPrimitiveComponent* Component) const { return ComponentMap.Contains(Component); }

	// Clears all internal references to the component and remove any associated record (e.g. FElements, Bounds). If RemovedRenderAssets is not null,
	// render assets with all components removed will be added to the array.
	void RemoveComponent(const IPrimitiveComponent* Component, FRemovedRenderAssetArray* RemovedRenderAssets);

	// Clears all internal references to the component. Dynamic instance state will also add an entry to the pending remove list which will be used
	// to remove the associated records (e.g. FElements, Bounds) when FlushPendingRemoveComponents is called.
	bool RemoveComponentReferences(const IPrimitiveComponent* Component);

	// Remove any record associated with pending remove components. Returns an array of render assets with all components removed.
	void FlushPendingRemoveComponents(FRemovedRenderAssetArray& RemovedRenderAssets);

	template<typename CallableT>
	void ForEachReferencedComponent(CallableT&& Callable) const
	{
		for (const TPair<const IPrimitiveComponent*, int32>& Component : ComponentMap)
		{
			Callable(Component.Key);
		}
	}

	void UpdateBounds(const IPrimitiveComponent* Component);
	bool UpdateBounds(int32 BoundIndex);
	bool ConditionalUpdateBounds(int32 BoundIndex);
	void UpdateLastRenderTimeAndMaxDrawDistance(int32 BoundIndex);

	uint32 GetAllocatedSize() const;

	// Generate the compiled elements.
	int32 CompileElements();
	int32 CheckRegistrationAndUnpackBounds(TArray<const IPrimitiveComponent*>& RemovedComponents);

	/** Move around one bound to free the last bound indices. This allows to keep the number of dynamic bounds low. */
	bool MoveBound(int32 SrcBoundIndex, int32 DstBoundIndex);
	void TrimBounds();
	void OffsetBounds(const FVector& Offset);

	FORCEINLINE int32 NumBounds() const { return Bounds4Components.Num(); }
	FORCEINLINE bool HasComponent(int32 BoundIndex) const { return Bounds4Components[BoundIndex] != nullptr; }

private:
	typedef int32 FRemovedComponentHandle;

	void RemoveComponentByHandle(FRemovedComponentHandle ElementIndex, FRemovedRenderAssetArray* RemovedRenderAssets);

	void AddElement(const IPrimitiveComponent* Component, const UStreamableRenderAsset* Asset, int BoundsIndex, float TexelFactor, bool bForceLoad, int32*& ComponentLink);
	// Returns the next elements using the same component.
	void RemoveElement(int32 ElementIndex, int32& NextComponentLink, int32& BoundsIndex, const UStreamableRenderAsset*& Asset);

	int32 AddBounds(const FBoxSphereBounds& Bounds, uint32 PackedRelativeBox, const IPrimitiveComponent* Component, float LastRenderTime, const FVector4& RangeOrigin, float MinDistanceSq, float MinRangeSq, float MaxRangeSq);
	FORCEINLINE int32 AddBounds(const IPrimitiveComponent* Component);
	void RemoveBounds(int32 Index);

	EAddComponentResult AddComponentInternal(const FPreAddComponentPayload& Payload);
	static EAddComponentResult AddComponentInternal(const IPrimitiveComponent* Component, FStreamingTextureLevelContext& LevelContext, float MaxAllowedUIDensity, FRenderAssetInstanceState* State, FPreAddComponentPayload* OutPreAddPayload);
	static EAddComponentResult AddComponentIgnoreBoundsInternal(const IPrimitiveComponent* Component, FStreamingTextureLevelContext& LevelContext, FRenderAssetInstanceState* State, FPreAddComponentIgnoreBoundsPayload* OutPreAddPayload);

	void AddRenderAssetElements(const IPrimitiveComponent* Component, const TArray<FPreAddRenderAssetElement>& RenderAssetElements, int32 BoundsIndex, int32*& ComponentLink);
	void AddRenderAssetElements(const IPrimitiveComponent* Component, const TArrayView<FStreamingRenderAssetPrimitiveInfo>& RenderAssetInstanceInfos, int32 BoundsIndex, int32*& ComponentLink);

private:

	const bool bIsDynamicInstanceState;

	/** 
	 * Components related to each of the Bounds4 elements. This is stored in another array to allow 
	 * passing Bounds4 to the threaded task without loosing the bound components, allowing incremental update.
	 */
	TArray<const IPrimitiveComponent*> Bounds4Components;

	TArray<int32> FreeBoundIndices;
	TArray<int32> FreeElementIndices;

	/** 
	 * When adding components that are not yet registered, bounds are not yet valid, and must be unpacked after the level becomes visible for the first time.
	 * We keep a list of bound require such unpacking as it would be risky to figure it out from the data itself. Some component data also shouldn't be unpacked
	 * if GetStreamingTextureInfo() returned entries with null PackedRelativeBox.
	 */
	TArray<int32>  BoundsToUnpack;

	/** Head element indices of removed components. Used to defer removal of associated FElements and Bounds. */
	TArray<FRemovedComponentHandle> PendingRemoveComponents;

	TMap<const IPrimitiveComponent*, int32> ComponentMap;

	friend class FRenderAssetLinkIterator;
	friend class FRenderAssetIterator;
};

template <typename TTasks>
class FRenderAssetInstanceStateTaskSync
{
public:

	FRenderAssetInstanceStateTaskSync()
		: FRenderAssetInstanceStateTaskSync(false)
	{}

	FORCEINLINE void Sync()
	{
		Tasks.SyncResults();
	}

	FORCEINLINE FRenderAssetInstanceState* SyncAndGetState()
	{
		Tasks.SyncResults();
		return State.GetReference();
	}

	// Get State but must be constant as async tasks could be reading data.
	FORCEINLINE const FRenderAssetInstanceState* GetState() const
	{
		return State.GetReference();
	}

	// Used when updating the state, but with no possible reallocation.
	FORCEINLINE FRenderAssetInstanceState* GetStateUnsafe()
	{
		return State.GetReference();
	}

	TTasks& GetTasks() { return Tasks; }
	const TTasks& GetTasks() const { return Tasks; }

protected:
	FRenderAssetInstanceStateTaskSync(bool bForDynamicInstances)
		: State(new FRenderAssetInstanceState(bForDynamicInstances))
	{}

	TRefCountPtr<FRenderAssetInstanceState> State;
	TTasks Tasks;
};

template <typename TTasks>
class FRenderAssetDynamicInstanceStateTaskSync : public FRenderAssetInstanceStateTaskSync<TTasks>
{
public:
	using Super = FRenderAssetInstanceStateTaskSync<TTasks>;

	DECLARE_DELEGATE_OneParam(FOnSyncDone, const FRemovedRenderAssetArray&);

	FRenderAssetDynamicInstanceStateTaskSync(FOnSyncDone&& InOnSyncDoneDelegate)
		: Super(true)
		, OnSyncDoneDelegate(InOnSyncDoneDelegate)
	{}

	void Sync()
	{
		check(IsInGameThread());

		Super::Tasks.SyncResults();
		FRemovedRenderAssetArray RemovedRenderAssets;
		Super::State->FlushPendingRemoveComponents(RemovedRenderAssets);
		OnSyncDoneDelegate.Execute(RemovedRenderAssets);
	}

	FRenderAssetInstanceState* SyncAndGetState()
	{
		Sync();
		return Super::State.GetReference();
	}

private:
	FOnSyncDone OnSyncDoneDelegate;
};
