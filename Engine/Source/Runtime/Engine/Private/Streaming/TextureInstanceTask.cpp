// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
TextureInstanceTask.cpp: Definitions of classes used for texture streaming.
=============================================================================*/

#include "Streaming/TextureInstanceTask.h"

#include "Streaming/SimpleStreamableAssetManager.h"

namespace RenderAssetInstanceTask {

FRefreshVisibility::FRefreshVisibility(const FOnWorkDone& InOnWorkDoneDelegate)
	: OnWorkDoneDelegate(InOnWorkDoneDelegate)
	, BeginIndex(INDEX_NONE)
	, EndIndex(INDEX_NONE)
{
}

void FRefreshVisibility::Init(FRenderAssetInstanceState* InState, int32 InBeginIndex, int32 InEndIndex)
{
	State = InState;
	BeginIndex = InBeginIndex;
	EndIndex = InEndIndex;
}

void FRefreshVisibility::operator()(bool bAsync)
{
	for (int32 Index = BeginIndex; Index < EndIndex; ++Index)
	{
		State->UpdateLastRenderTimeAndMaxDrawDistance(Index);
	}
}

void FRefreshVisibility::Sync()
{ 
	OnWorkDoneDelegate.Execute(BeginIndex, EndIndex); 
	State.SafeRelease(); // This must be done on the game thread as ref counters are not thread safe.
}

FRefreshFull::FRefreshFull(const FOnWorkDone& InOnWorkDoneDelegate)
	: OnWorkDoneDelegate(InOnWorkDoneDelegate)
	, FirstFreeBound(INDEX_NONE)
	, LastUsedBound(INDEX_NONE)
	, BeginIndex(INDEX_NONE)
	, EndIndex(INDEX_NONE)
{
}

void FRefreshFull::Init(FRenderAssetInstanceState* InState, int32 InBeginIndex, int32 InEndIndex)
{
	check(!SkippedIndices.Num());

	State = InState;
	FirstFreeBound = INDEX_NONE;
	LastUsedBound = INDEX_NONE;
	BeginIndex = InBeginIndex;
	EndIndex = InEndIndex;
}

void FRefreshFull::operator()(bool bAsync)
{
	if (BeginIndex > 0 && !State->HasComponent(BeginIndex - 1))
	{	// If there is any free spaces in the previous update range, the defrag process will move then toward the tail.
		// To allow bounds to move between update range, we check if the last element is a free element.
		FirstFreeBound = BeginIndex - 1;
	}

	if (bAsync)
	{
		for (int32 Index = BeginIndex; Index < EndIndex; ++Index)
		{
			if (State->ConditionalUpdateBounds(Index))
			{
				LastUsedBound = Index;
			}
			else
			{
				if (State->HasComponent(Index))
				{
					SkippedIndices.Add(Index); // Update this later
					LastUsedBound = Index;
				}
				else if (FirstFreeBound == INDEX_NONE)
				{
					FirstFreeBound = Index;
				}
			}
		}
	}
	else
	{
		for (int32 Index = BeginIndex; Index < EndIndex; ++Index)
		{
			if (State->UpdateBounds(Index))
			{
				LastUsedBound = Index;
			}
			else if (FirstFreeBound == INDEX_NONE)
			{
				FirstFreeBound = Index;
			}
		}
	}
}

void FRefreshFull::Sync()
{ 
	OnWorkDoneDelegate.Execute(BeginIndex, EndIndex, SkippedIndices, FirstFreeBound, LastUsedBound);
	State.SafeRelease(); // This must be done on the game thread as ref counters are not thread safe.
	SkippedIndices.Reset();
}

void FNormalizeLightmapTexelFactor::operator()(bool bAsync)
{
	TArray<float> TexelFactors;
	for (auto TextureIt = State->GetRenderAssetIterator(); TextureIt; ++TextureIt)
	{
		const UStreamableRenderAsset* Texture = *TextureIt;
		const int32 TextureLODGroup = TextureIt.GetLODGroup();
		if (TextureLODGroup == TEXTUREGROUP_Lightmap || TextureLODGroup == TEXTUREGROUP_Shadowmap)
		{
			TexelFactors.Reset();
			for (auto ElementIt = State->GetElementIterator(Texture); ElementIt ; ++ElementIt)
			{
				TexelFactors.Push(ElementIt.GetTexelFactor());
			}
			TexelFactors.Sort();

			const float MinTexelFactor = TexelFactors[FMath::FloorToInt(TexelFactors.Num() * 0.2f)];
			const float MaxTexelFactor = TexelFactors[FMath::FloorToInt(TexelFactors.Num() * 0.8f)];

			for (auto ElementIt = State->GetElementIterator(Texture); ElementIt ; ++ElementIt)
			{
				ElementIt.ClampTexelFactor(MinTexelFactor, MaxTexelFactor);
			}
		}
	}
}

void FCreateViewWithUninitializedBounds::operator()(bool bAsync)
{ 
	// The task creates another view but does not fully initialize the bounds.
	// At sync point, the bound array are swapped as the dirty array gets refreshed.
	View = FRenderAssetInstanceView::CreateViewWithUninitializedBounds(State.GetReference());

	// Release the previous view. Ran here because the destructor is super expansive.
	ViewToRelease.SafeRelease();
}

void FCreateViewWithUninitializedBounds::Sync()
{ 
	OnWorkDoneDelegate.Execute(View.GetReference()); 
	View.SafeRelease();
	State.SafeRelease();
	ViewToRelease.SafeRelease();
}

template <typename TTask> 
void FDoWorkTask::ProcessTasks(TArray< TRefCountPtr<TTask> >& Tasks)
{
	for (int32 Index = 0; Index < Tasks.Num(); ++Index)
	{
		TTask* Task = Tasks[Index].GetReference();

		if (Task->GetRefCount() == 1)
		{	 // Release it since this is the only reference.
			Tasks.RemoveAtSwap(Index);
			--Index;
		}
		else
		{
			Task->TryWork(true);
		}
	}
}

void FDoWorkTask::DoWork()
{
	LLM_SCOPE(ELLMTag::Textures);
	TRACE_CPUPROFILER_EVENT_SCOPE(RenderAssetInstanceTask);
	
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FSimpleStreamableAssetManager);
		FSimpleStreamableAssetManager::Process();
	}

	// The order of execution is important in case other tasks need to sync.
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(RefreshFull);
		ProcessTasks(RefreshFullTasks);
	}
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(CreateViewWithUninitializedBounds);
		ProcessTasks(CreateViewWithUninitializedBoundsTasks);
	}
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(RefreshVisibility);
		ProcessTasks(RefreshVisibilityTasks);
	}
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(NormalizeLightmapTexelFactor);
		ProcessTasks(NormalizeLightmapTexelFactorTasks);
	}
}

} // namespace RenderAssetInstanceTask
