// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
TextureInstanceTask.h: Definitions of classes used for texture streaming.
=============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "Streaming/TextureInstanceState.h"
#include "Async/AsyncWork.h"

namespace RenderAssetInstanceTask {

enum ETaskState
{
	TS_Done,
	TS_WorkPending,
	TS_WorkInProgress,
	TS_SyncPending
};

template <typename TWork>
class TDoWorkTask : public UE::Private::FQueryableRefCountedObject
{
public:

	/** Forwarding constructor. */
	template<typename...T> 
	explicit TDoWorkTask(T&&... Args) : Work(Forward<T>(Args)...), TaskState(TS_Done) {}

	virtual ~TDoWorkTask() = default;

	// Whether to run the work
	FORCEINLINE_DEBUGGABLE void TryWork(bool bAsync) 
	{ 
		// Force inline as this is only called from a single place.
		if (FPlatformAtomics::InterlockedCompareExchange(&TaskState, TS_WorkInProgress, TS_WorkPending) == TS_WorkPending)
		{
			Work(bAsync);
			TaskState = TS_SyncPending;
		}
	}

	// Force inline as this is only called from a single place.
	FORCEINLINE_DEBUGGABLE void TrySync()
	{ 
		if (TaskState != TS_Done)
		{
			while (TaskState != TS_SyncPending)
			{
				FPlatformProcess::Sleep(0);
			}
			Work.Sync();
		}
		TaskState = TS_Done;
	}

	/** Forwarding constructor. */
	template<typename...T> 
	FORCEINLINE void Init(T&&... Args)
	{
		check(TaskState == TS_Done);
		Work.Init(Forward<T>(Args)...);
		TaskState = TS_WorkPending;
	}

private:

	TWork Work;
	volatile int32 TaskState;
};

/** Refresh component visibility. */
class FRefreshVisibility
{
public:

	DECLARE_DELEGATE_TwoParams(FOnWorkDone, int32, int32);

	FRefreshVisibility(const FOnWorkDone& InOnWorkDoneDelegate);
	void Init(FRenderAssetInstanceState* InState, int32 InBeginIndex, int32 InEndIndex);
	void operator()(bool bAsync);
	void Sync();

private:

	// Callback to process the results.
	FOnWorkDone OnWorkDoneDelegate;
	// The state to update (no re/allocation allowed)
	TRefCountPtr<FRenderAssetInstanceState> State;
	// The index of the first bound to update.
	int32 BeginIndex;
	// The index of the last bound to update.
	int32 EndIndex;
};

/** Refresh all component data. */
class FRefreshFull
{
public:
	
	DECLARE_DELEGATE_FiveParams(FOnWorkDone, int32, int32, const TArray<int32>&, int32, int32);

	FRefreshFull(const FOnWorkDone& InOnWorkDoneDelegate);
	void Init(FRenderAssetInstanceState* InState, int32 InBeginIndex, int32 InEndIndex);
	void operator()(bool bAsync);
	void Sync();

private:

	// Callback to process the results.
	FOnWorkDone OnWorkDoneDelegate;
	// The first free bound seen (used for defrag)
	int32 FirstFreeBound = INDEX_NONE;
	// The last free bound seen (used for defrag)
	int32 LastUsedBound = INDEX_NONE;
	// Any bounds that couldn't be updated for some reason (incoherent bounds).
	TArray<int32> SkippedIndices;
	// The state to update (no re/allocation allowed)
	TRefCountPtr<FRenderAssetInstanceState> State;
	// The index of the first bound to update.
	int32 BeginIndex;
	// The index of the last bound to update.
	int32 EndIndex;
};

/** Normalize the texel factors within state to reduce extremas. */
class FNormalizeLightmapTexelFactor
{
public:

	void Init(FRenderAssetInstanceState* InState) { State = InState; }
	void operator()(bool bAsync);
	void Sync() { State.SafeRelease(); }

private:

	// The state to update (no re/allocation allowed)
	TRefCountPtr<FRenderAssetInstanceState> State;
};

// Create an independent view of a state
class FCreateViewWithUninitializedBounds : public FNonAbandonableTask
{
public:
	DECLARE_DELEGATE_OneParam(FOnWorkDone, FRenderAssetInstanceView*);

	FCreateViewWithUninitializedBounds(const FOnWorkDone& InOnWorkDoneDelegate) : OnWorkDoneDelegate(InOnWorkDoneDelegate) {}
	void Init(const FRenderAssetInstanceState* InState, const FRenderAssetInstanceView* InViewToRelease)
	{
		State = InState;
		ViewToRelease = InViewToRelease;
	}

	void operator()(bool bAsync);
	void Sync();

private:

	// Callback to process the results.
	FOnWorkDone OnWorkDoneDelegate;
	// The view created from the state, as a result of the execution.
	TRefCountPtr<FRenderAssetInstanceView> View;
	// The state for which to create the view.
	TRefCountPtr<const FRenderAssetInstanceState> State;
	// The previous view of the state. Used to release the state and run the destructor async.
	TRefCountPtr<const FRenderAssetInstanceView> ViewToRelease;
};

typedef TDoWorkTask<FRefreshFull> FRefreshFullTask;
typedef TDoWorkTask<FRefreshVisibility> FRefreshVisibilityTask;
typedef TDoWorkTask<FNormalizeLightmapTexelFactor> FNormalizeLightmapTexelFactorTask;
typedef TDoWorkTask<FCreateViewWithUninitializedBounds> FCreateViewWithUninitializedBoundsTask;

class FDoWorkTask : public FNonAbandonableTask
{
public:

	void DoWork();

	void Add(FRefreshFullTask* RefreshFullTask) { RefreshFullTasks.Add(RefreshFullTask); }
	void Add(FRefreshVisibilityTask* RefreshVisibilityTask) { RefreshVisibilityTasks.Add(RefreshVisibilityTask); }
	void Add(FNormalizeLightmapTexelFactorTask* NormalizeLightmapTexelFactorTask) { NormalizeLightmapTexelFactorTasks.Add(NormalizeLightmapTexelFactorTask); }
	void Add(FCreateViewWithUninitializedBoundsTask* CreateViewWithUninitializedBoundsTask) { CreateViewWithUninitializedBoundsTasks.Add(CreateViewWithUninitializedBoundsTask); }

	FORCEINLINE TStatId GetStatId() const { RETURN_QUICK_DECLARE_CYCLE_STAT(FCreateViewWithUninitializedBounds, STATGROUP_ThreadPoolAsyncTasks); }

private:

	template <typename TTask> 
	void ProcessTasks(TArray< TRefCountPtr<TTask> >& Tasks);

	TArray< TRefCountPtr<FRefreshFullTask> > RefreshFullTasks;
	TArray< TRefCountPtr<FRefreshVisibilityTask> > RefreshVisibilityTasks;
	TArray< TRefCountPtr<FNormalizeLightmapTexelFactorTask> > NormalizeLightmapTexelFactorTasks;
	TArray< TRefCountPtr<FCreateViewWithUninitializedBoundsTask> > CreateViewWithUninitializedBoundsTasks;
};

class FDoWorkAsyncTask : public FAsyncTask<FDoWorkTask>, public FRefCountedObject
{
};

} // namespace RenderAssetInstanceTask
