// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	TextureInstanceView.cpp: Implementation of content streaming classes.
=============================================================================*/

#include "Streaming/TextureInstanceView.h"
#include "Streaming/TextureInstanceView.inl"
#include "Streaming/TextureStreamingHelpers.h"
#include "Streaming/SimpleStreamableAssetManager.h"
#include "Components/PrimitiveComponent.h"
#include "ContentStreaming.h"

void FBounds4::Set(int32 Index, const FBoxSphereBounds& InBounds, uint32 InPackedRelativeBox, float InLastRenderTime, const FVector& InRangeOrigin, float InMinDistanceSq, float InMinRangeSq, float InMaxRangeSq)
{
	Set(Index, InBounds, InPackedRelativeBox, InBounds.SphereRadius, InLastRenderTime, InRangeOrigin, InMinDistanceSq, InMinRangeSq, InMaxRangeSq);
}
void FBounds4::Set(int32 Index, const FBoxSphereBounds& InBounds, uint32 InPackedRelativeBox, float InRadiusOrComponentScale, float InLastRenderTime, const FVector& InRangeOrigin, float InMinDistanceSq, float InMinRangeSq, float InMaxRangeSq)
{
	check(Index >= 0 && Index < 4);

	OriginX.Component(Index) = InBounds.Origin.X;
	OriginY.Component(Index) = InBounds.Origin.Y;
	OriginZ.Component(Index) = InBounds.Origin.Z;
	RangeOriginX.Component(Index) = InRangeOrigin.X;
	RangeOriginY.Component(Index) = InRangeOrigin.Y;
	RangeOriginZ.Component(Index) = InRangeOrigin.Z;
	ExtentX.Component(Index) = InBounds.BoxExtent.X;
	ExtentY.Component(Index) = InBounds.BoxExtent.Y;
	ExtentZ.Component(Index) = InBounds.BoxExtent.Z;
	RadiusOrComponentScale.Component(Index) = InRadiusOrComponentScale;
	PackedRelativeBox[Index] = InPackedRelativeBox;
	MinDistanceSq.Component(Index) = InMinDistanceSq;
	MinRangeSq.Component(Index) = InMinRangeSq;
	MaxRangeSq.Component(Index) = InMaxRangeSq;
	LastRenderTime.Component(Index) = InLastRenderTime;
}

void FBounds4::UnpackBounds(int32 Index, const IPrimitiveComponent* Component)
{
	check(Component);
	check(Index >= 0 && Index < 4);

	if (PackedRelativeBox[Index])
	{
		const FBoxSphereBounds& ComponentBounds = Component->GetBounds();

		FBoxSphereBounds SubBounds;
		UnpackRelativeBox(ComponentBounds, PackedRelativeBox[Index], SubBounds);

		// Update the visibility range once we have the bounds.
		float MinDistance2 = 0, MinRange2 = 0, MaxRange2 = FLT_MAX;
		FRenderAssetInstanceView::GetDistanceAndRange(Component, SubBounds, MinDistance2, MinRange2, MaxRange2);

		OriginX.Component(Index) = SubBounds.Origin.X;
		OriginY.Component(Index) = SubBounds.Origin.Y;
		OriginZ.Component(Index) = SubBounds.Origin.Z;
		RangeOriginX.Component(Index) = ComponentBounds.Origin.X;
		RangeOriginY.Component(Index) = ComponentBounds.Origin.Y;
		RangeOriginZ.Component(Index) = ComponentBounds.Origin.Z;
		ExtentX.Component(Index) = SubBounds.BoxExtent.X;
		ExtentY.Component(Index) = SubBounds.BoxExtent.Y;
		ExtentZ.Component(Index) = SubBounds.BoxExtent.Z;
		RadiusOrComponentScale.Component(Index) = SubBounds.SphereRadius;
		PackedRelativeBox[Index] = PackedRelativeBox_Identity;
		MinDistanceSq.Component(Index) = MinDistance2;
		MinRangeSq.Component(Index) = MinRange2;
		MaxRangeSq.Component(Index) = MaxRange2;
	}
}

/** Dynamic Path, this needs to reset all members since the dynamic data is rebuilt from scratch every update (the previous data is given to the async task) */
void FBounds4::FullUpdate(int32 Index, const FVector& NewOrigin, const FVector& NewBoxExtent, float NewSphereRadius, float InLastRenderTime)
{
	check(Index >= 0 && Index < 4);

	OriginX.Component(Index) = NewOrigin.X;
	OriginY.Component(Index) = NewOrigin.Y;
	OriginZ.Component(Index) = NewOrigin.Z;
	RangeOriginX.Component(Index) = NewOrigin.X;
	RangeOriginY.Component(Index) = NewOrigin.Y;
	RangeOriginZ.Component(Index) = NewOrigin.Z;
	ExtentX.Component(Index) = NewBoxExtent.X;
	ExtentY.Component(Index) = NewBoxExtent.Y;
	ExtentZ.Component(Index) = NewBoxExtent.Z;
	RadiusOrComponentScale.Component(Index) = NewSphereRadius;
	PackedRelativeBox[Index] = PackedRelativeBox_Identity;
	MinDistanceSq.Component(Index) = 0;
	MinRangeSq.Component(Index) = 0;
	MaxRangeSq.Component(Index) = FLT_MAX;
	LastRenderTime.Component(Index) = InLastRenderTime;
}

FRenderAssetInstanceView::FRenderAssetLinkConstIterator::FRenderAssetLinkConstIterator(const FRenderAssetInstanceView& InState, const UStreamableRenderAsset* InAsset)
:	State(InState)
,	CurrElementIndex(INDEX_NONE)
{
	const FRenderAssetDesc* AssetDesc = State.RenderAssetMap.Find(InAsset);
	if (AssetDesc)
	{
		CurrElementIndex = AssetDesc->HeadLink;
	}
}

FBoxSphereBounds FRenderAssetInstanceView::FRenderAssetLinkConstIterator::GetBounds() const
{ 
	FBoxSphereBounds Bounds(ForceInitToZero);

	int32 BoundsIndex = State.Elements[CurrElementIndex].BoundsIndex; 
	if (State.Bounds4.IsValidIndex(BoundsIndex / 4))
	{
		const FBounds4& TheBounds4 = State.Bounds4[BoundsIndex / 4];
		int32 Index = BoundsIndex % 4;

		Bounds.Origin.X = TheBounds4.OriginX[Index];
		Bounds.Origin.Y = TheBounds4.OriginY[Index];
		Bounds.Origin.Z = TheBounds4.OriginZ[Index];

		Bounds.BoxExtent.X = TheBounds4.ExtentX[Index];
		Bounds.BoxExtent.Y = TheBounds4.ExtentY[Index];
		Bounds.BoxExtent.Z = TheBounds4.ExtentZ[Index];

		Bounds.SphereRadius = Bounds.BoxExtent.Length();
	}
	return Bounds;
}

void FRenderAssetInstanceView::FRenderAssetLinkConstIterator::OutputToLog(float MaxNormalizedSize, float MaxNormalizedSize_VisibleOnly, const TCHAR* Prefix) const
{
#if !UE_BUILD_SHIPPING
	FBoxSphereBounds Bounds = GetBounds();
	FRenderAssetInstanceView::OutputToLog(Bounds, GetBoundsIndex(), State.Bounds4, GetTexelFactor(), GetForceLoad(), MaxNormalizedSize, MaxNormalizedSize_VisibleOnly, Prefix, GetComponent());
#endif
}

void FRenderAssetInstanceView::OutputToLog(const FBoxSphereBounds& Bounds, int32 BoundsIndex, const TArray<FBounds4>& Bounds4, float TexelFactor, bool bForceLoad, float MaxNormalizedSize, float MaxNormalizedSize_VisibleOnly, const TCHAR* Prefix, const IPrimitiveComponent* Component)
{
#if !UE_BUILD_SHIPPING
	// Log the component reference.
	if (Component)
	{
		UE_LOGF(LogContentStreaming, Log, "  %lsReference= %ls", Prefix, *Component->GetFullName());
	}
	else
	{
		UE_LOGF(LogContentStreaming, Log, "  %lsReference", Prefix);
	}
	
	// Log the wanted mips.
	if (TexelFactor == FLT_MAX || bForceLoad)
	{
		UE_LOGF(LogContentStreaming, Log, "    Forced FullyLoad");
	}
	else if (TexelFactor >= 0)
	{
		if (GIsEditor) // In editor, visibility information is unreliable and we only consider the max.
		{
			UE_LOGF(LogContentStreaming, Log, "    Size=%f, BoundIndex=%d", TexelFactor * FMath::Max(MaxNormalizedSize, MaxNormalizedSize_VisibleOnly), BoundsIndex);
		}
		else if (MaxNormalizedSize_VisibleOnly > 0)
		{
			UE_LOGF(LogContentStreaming, Log, "    OnScreenSize=%f, BoundIndex=%d", TexelFactor * MaxNormalizedSize_VisibleOnly, BoundsIndex);
		}
		else
		{
			if (Bounds4.IsValidIndex(BoundsIndex / 4))
			{
				float LastRenderTime = Bounds4[BoundsIndex / 4].LastRenderTime[BoundsIndex % 4];
				UE_LOGF(LogContentStreaming, Log, "    OffScreenSize=%f, LastRenderTime= %.3f, BoundIndex=%d", TexelFactor * MaxNormalizedSize, LastRenderTime, BoundsIndex);
			}
			else
			{
				UE_LOGF(LogContentStreaming, Log, "    OffScreenSize=%f, BoundIndex=Invalid", TexelFactor * MaxNormalizedSize);
			}
		}
	}
	else // Negative texel factors relate to forced specific resolution.
	{
		UE_LOGF(LogContentStreaming, Log, "    ForcedSize=%f", -TexelFactor);
	}

	// Log the bounds
	if (CVarStreamingUseNewMetrics.GetValueOnGameThread() != 0) // New metrics uses AABB while previous metrics used spheres.
	{
		if (TexelFactor >= 0 && TexelFactor < FLT_MAX)
		{
			UE_LOGF(LogContentStreaming, Log, "    Origin=(%ls), BoxExtent=(%ls), TexelSize=%f", *Bounds.Origin.ToString(), *Bounds.BoxExtent.ToString(), TexelFactor);
		}
		else
		{
			UE_LOGF(LogContentStreaming, Log, "    Origin=(%ls), BoxExtent=(%ls)", *Bounds.Origin.ToString(), *Bounds.BoxExtent.ToString());
		}
	}
	else
	{
		if (TexelFactor >= 0 && TexelFactor < FLT_MAX)
		{
			UE_LOGF(LogContentStreaming, Log, "    Origin=(%ls), SphereRadius=%f, TexelSize=%f",  *Bounds.Origin.ToString(), Bounds.SphereRadius, TexelFactor);
		}
		else
		{
			UE_LOGF(LogContentStreaming, Log, "    Origin=(%ls), SphereRadius=%f",  *Bounds.Origin.ToString(), Bounds.SphereRadius);
		}
	}
#endif // !UE_BUILD_SHIPPING
}

TRefCountPtr<const FRenderAssetInstanceView> FRenderAssetInstanceView::CreateView(const FRenderAssetInstanceView* RefView)
{
	TRefCountPtr<FRenderAssetInstanceView> NewView(new FRenderAssetInstanceView());
	
	NewView->Bounds4 = RefView->Bounds4;
	NewView->Elements = RefView->Elements;
	NewView->RenderAssetMap = RefView->RenderAssetMap;
	NewView->MaxTexelFactor = RefView->MaxTexelFactor;
	// NewView->CompiledTextureMap = RefView->CompiledTextureMap;

	return TRefCountPtr<const FRenderAssetInstanceView>(NewView.GetReference());
}

TRefCountPtr<FRenderAssetInstanceView> FRenderAssetInstanceView::CreateViewWithUninitializedBounds(const FRenderAssetInstanceView* RefView)
{
	TRefCountPtr<FRenderAssetInstanceView> NewView(new FRenderAssetInstanceView());
	
	NewView->Bounds4.AddUninitialized(RefView->Bounds4.Num());
	NewView->Elements = RefView->Elements;
	NewView->RenderAssetMap = RefView->RenderAssetMap;
	NewView->MaxTexelFactor = RefView->MaxTexelFactor;
	// NewView->CompiledTextureMap = RefView->RefView;

	return NewView;
}

float FRenderAssetInstanceView::GetMaxDrawDistSqWithLODParent(const FVector& Origin, const FVector& ParentOrigin, float ParentMinDrawDist, float ParentBoundingSphereRadius)
{
	const float Result = ParentMinDrawDist + ParentBoundingSphereRadius + (Origin - ParentOrigin).Size();
	return Result * Result;
}

void FRenderAssetInstanceView::GetDistanceAndRange(const IPrimitiveComponent* Component, const FBoxSphereBounds& RenderAssetInstanceBounds, float& MinDistanceSq, float& MinRangeSq, float& MaxRangeSq)
{
	check(Component && Component->IsRegistered());

	// In the engine, the MinDistance is computed from the component bound center to the viewpoint (except for HLODs).
	// The streaming computes the distance as the distance from viewpoint to the edge of the texture bound box.
	// The implementation also handles MinDistance by bounding the distance to it so that if the viewpoint gets closer the screen size will stop increasing at some point.
	// The fact that the primitive will disappear is not so relevant as this will be handled by the visibility logic, normally streaming one less mip than requested.
	// The important mater is to control the requested mip by limiting the distance, since at close up, the distance becomes very small and all mips are streamer (even after the 1 mip bias).

	const IPrimitiveComponent* LODParent = Component->GetLODParentPrimitive();

	MinDistanceSq = FMath::Max<float>(0, Component->GetMinDrawDistance() - (RenderAssetInstanceBounds.Origin - Component->GetBounds().Origin).Size() - RenderAssetInstanceBounds.SphereRadius);
	MinDistanceSq *= MinDistanceSq;
	MinRangeSq = FMath::Max<float>(0, Component->GetMinDrawDistance());
	MinRangeSq *= MinRangeSq;
	// Max distance when HLOD becomes visible.
	MaxRangeSq = LODParent ? GetMaxDrawDistSqWithLODParent(Component->GetBounds().Origin, LODParent->GetBounds().Origin, LODParent->GetMinDrawDistance(), LODParent->GetBounds().SphereRadius) : FLT_MAX;
}

void FRenderAssetInstanceView::SwapData(FRenderAssetInstanceView* Lfs, FRenderAssetInstanceView* Rhs)
{
	// Things must be compatible somehow or derived classes will be in incoherent state.
	check(Lfs->Bounds4.Num() == Rhs->Bounds4.Num());
	check(Lfs->Elements.Num() == Rhs->Elements.Num());
	check(Lfs->RenderAssetMap.Num() == Rhs->RenderAssetMap.Num());
	check(Lfs->CompiledRenderAssetMap.Num() == 0 && Rhs->CompiledRenderAssetMap.Num() == 0);
	check(!Lfs->CompiledNumForcedLODCompMap.Num() && !Rhs->CompiledNumForcedLODCompMap.Num());

	Swap(Lfs->Bounds4 , Rhs->Bounds4);
	Swap(Lfs->Elements , Rhs->Elements);
	Swap(Lfs->RenderAssetMap, Rhs->RenderAssetMap);
	Swap(Lfs->MaxTexelFactor , Rhs->MaxTexelFactor);
}

void FRenderAssetInstanceAsyncView::UpdateBoundSizes_Async(
	const TArray<FStreamingViewInfo>& ViewInfos,
	const FStreamingViewInfoExtraArray& ViewInfoExtras,
	float LastUpdateTime,
	const FRenderAssetStreamingSettings& Settings)
{
	if (!View.IsValid())
	{
		return;
	}
	
	UpdateBoundSizes(ViewInfos,ViewInfoExtras,
		LastUpdateTime,
		View->GetMaxTexelFactor(),
		Settings,
		View->GetBoundsView(),
		BoundsViewInfo,
		MaxLevelRenderAssetScreenSize);
}

void FRenderAssetInstanceAsyncView::UpdateBoundSizes(
	const TArray<FStreamingViewInfo>& ViewInfos,
	const FStreamingViewInfoExtraArray& ViewInfoExtras,
	float LastUpdateTime,
	float MaxTexelFactor,
	const FRenderAssetStreamingSettings& Settings,
	const TConstArrayView<FBounds4>& Bounds4,
	TArray<FBoundsViewInfo>& OutBoundsViewInfos,
	float& OutMaxLevelRenderAssetScreenSize)
{
	check(ViewInfos.Num() == ViewInfoExtras.Num());

	const int32 NumViews = ViewInfos.Num();
	const int32 NumBounds4 = Bounds4.Num();

	const bool bConsiderVisibility = !FSimpleStreamableAssetManager::IsEnabled() || FSimpleStreamableAssetManager::ShouldConsiderVisibility();
	const VectorRegister LastUpdateTime4 = VectorSet(LastUpdateTime, LastUpdateTime, LastUpdateTime, LastUpdateTime);

	OutBoundsViewInfos.Empty(NumBounds4 * 4);
	OutBoundsViewInfos.AddUninitialized(NumBounds4 * 4);

	// Max normalized size from all elements
	VectorRegister ViewMaxNormalizedSize = VectorZero();

	for (int32 Bounds4Index = 0; Bounds4Index < NumBounds4; ++Bounds4Index)
	{
		const FBounds4& CurrentBounds4 = Bounds4[Bounds4Index];

		// LWC_TODO - Origin values are loaded from doubles, the remaining values are loaded from floats
		// Could potentially perform some of these operations with float VectorRegisters, which could potentially be more efficient
		// (Otherwise we're paying cost to convert these values to double VectorRegisters on load)
		// Tricky to manage precision though, as with large worlds, distance between object and view origin can potentially overflow float capacity

		// Calculate distance of viewer to bounding sphere.
		const VectorRegister OriginX = VectorLoadAligned( &CurrentBounds4.OriginX );
		const VectorRegister OriginY = VectorLoadAligned( &CurrentBounds4.OriginY );
		const VectorRegister OriginZ = VectorLoadAligned( &CurrentBounds4.OriginZ );
		const VectorRegister RangeOriginX = VectorLoadAligned( &CurrentBounds4.RangeOriginX );
		const VectorRegister RangeOriginY = VectorLoadAligned( &CurrentBounds4.RangeOriginY );
		const VectorRegister RangeOriginZ = VectorLoadAligned( &CurrentBounds4.RangeOriginZ );
		const VectorRegister ExtentX = VectorLoadAligned( &CurrentBounds4.ExtentX );
		const VectorRegister ExtentY = VectorLoadAligned( &CurrentBounds4.ExtentY );
		const VectorRegister ExtentZ = VectorLoadAligned( &CurrentBounds4.ExtentZ );
		const VectorRegister ComponentScale = VectorLoadAligned( &CurrentBounds4.RadiusOrComponentScale );
		const VectorRegister PackedRelativeBox = VectorLoadAligned( reinterpret_cast<const FVector4f*>(&CurrentBounds4.PackedRelativeBox) );
		const VectorRegister MinDistanceSq = VectorLoadAligned( &CurrentBounds4.MinDistanceSq );
		const VectorRegister MinRangeSq = VectorLoadAligned( &CurrentBounds4.MinRangeSq );
		const VectorRegister MaxRangeSq = VectorLoadAligned(&CurrentBounds4.MaxRangeSq);
		const VectorRegister LastRenderTime = VectorLoadAligned(&CurrentBounds4.LastRenderTime);

		VectorRegister MaxNormalizedSize = VectorZero();
		VectorRegister MaxNormalizedSize_VisibleOnly = VectorZero();

		for (int32 ViewIndex = 0; ViewIndex < NumViews; ++ViewIndex)
		{
			const FStreamingViewInfo& ViewInfo = ViewInfos[ViewIndex];
			const FStreamingViewInfoExtra& ViewInfoExtra = ViewInfoExtras[ViewIndex];

			const VectorRegister ScreenSize = VectorLoadFloat1( &ViewInfoExtra.ScreenSizeFloat );
			const VectorRegister ExtraBoostForVisiblePrimitive = VectorLoadFloat1( &ViewInfoExtra.ExtraBoostForVisiblePrimitiveFloat );
			const VectorRegister ViewOriginX = VectorLoadFloat1( &ViewInfo.ViewOrigin.X );
			const VectorRegister ViewOriginY = VectorLoadFloat1( &ViewInfo.ViewOrigin.Y );
			const VectorRegister ViewOriginZ = VectorLoadFloat1( &ViewInfo.ViewOrigin.Z );

			VectorRegister DistSqMinusRadiusSq = VectorZero();
			if (Settings.bUseNewMetrics)
			{
				// In this case DistSqMinusRadiusSq will contain the distance to the box^2
				VectorRegister Temp = VectorSubtract( ViewOriginX, OriginX );
				Temp = VectorAbs( Temp );
				VectorRegister BoxRef = VectorMin( Temp, ExtentX );
				Temp = VectorSubtract( Temp, BoxRef );
				DistSqMinusRadiusSq = VectorMultiply( Temp, Temp );

				Temp = VectorSubtract( ViewOriginY, OriginY );
				Temp = VectorAbs( Temp );
				BoxRef = VectorMin( Temp, ExtentY );
				Temp = VectorSubtract( Temp, BoxRef );
				DistSqMinusRadiusSq = VectorMultiplyAdd( Temp, Temp, DistSqMinusRadiusSq );

				Temp = VectorSubtract( ViewOriginZ, OriginZ );
				Temp = VectorAbs( Temp );
				BoxRef = VectorMin( Temp, ExtentZ );
				Temp = VectorSubtract( Temp, BoxRef );
				DistSqMinusRadiusSq = VectorMultiplyAdd( Temp, Temp, DistSqMinusRadiusSq );
			}
			else
			{
				VectorRegister Temp = VectorSubtract( ViewOriginX, OriginX );
				VectorRegister DistSq = VectorMultiply( Temp, Temp );
				Temp = VectorSubtract( ViewOriginY, OriginY );
				DistSq = VectorMultiplyAdd( Temp, Temp, DistSq );
				Temp = VectorSubtract( ViewOriginZ, OriginZ );
				DistSq = VectorMultiplyAdd( Temp, Temp, DistSq );

				DistSqMinusRadiusSq = VectorNegateMultiplyAdd( ExtentX, ExtentX, DistSq );
				DistSqMinusRadiusSq = VectorNegateMultiplyAdd( ExtentY, ExtentY, DistSq );
				DistSqMinusRadiusSq = VectorNegateMultiplyAdd( ExtentZ, ExtentZ, DistSq );
				// This can be negative here!!!
			}

			// If the bound is not visible up close, limit the distance to it's minimal possible range.
			VectorRegister ClampedDistSq = VectorMax( MinDistanceSq, DistSqMinusRadiusSq );

			// Compute in range  Squared distance between range.
			VectorRegister InRangeMask;
			{
				VectorRegister Temp = VectorSubtract( ViewOriginX, RangeOriginX );
				VectorRegister RangeDistSq = VectorMultiply( Temp, Temp );
				Temp = VectorSubtract( ViewOriginY, RangeOriginY );
				RangeDistSq = VectorMultiplyAdd( Temp, Temp, RangeDistSq );
				Temp = VectorSubtract( ViewOriginZ, RangeOriginZ );
				RangeDistSq = VectorMultiplyAdd( Temp, Temp, RangeDistSq );

				VectorRegister ClampedRangeDistSq = VectorMax( MinRangeSq, RangeDistSq );
				ClampedRangeDistSq = VectorMin( MaxRangeSq, ClampedRangeDistSq );
				InRangeMask = VectorCompareEQ( RangeDistSq, ClampedRangeDistSq); // If the clamp dist is equal, then it was in range.
			}

			ClampedDistSq = VectorMax(ClampedDistSq, VectorOne()); // Prevents / 0
			VectorRegister ScreenSizeOverDistance = VectorReciprocalSqrt(ClampedDistSq);
			ScreenSizeOverDistance = VectorMultiply(ScreenSizeOverDistance, ScreenSize);

			MaxNormalizedSize = VectorMax(ScreenSizeOverDistance, MaxNormalizedSize);

			// Accumulate the view max amongst all. When PackedRelativeBox == 0, the entry is not valid and must not affet the max.
			const VectorRegister CulledMaxNormalizedSize = VectorSelect(VectorCompareNE(PackedRelativeBox, VectorZero()), MaxNormalizedSize, VectorZero());
			ViewMaxNormalizedSize = VectorMax(ViewMaxNormalizedSize, CulledMaxNormalizedSize);

			// Now mask to zero if not in range, or not seen recently.
			ScreenSizeOverDistance = VectorMultiply(ScreenSizeOverDistance, ExtraBoostForVisiblePrimitive);
			ScreenSizeOverDistance = VectorSelect(InRangeMask, ScreenSizeOverDistance, VectorZero());
			// When using SimpleStreamableAssetManager, LastRenderTime is only valid when FSimpleStreamableAssetManager::ShouldConsiderVisibility() == true
			if (bConsiderVisibility)
			{
				ScreenSizeOverDistance = VectorSelect(VectorCompareGT(LastRenderTime, LastUpdateTime4), ScreenSizeOverDistance, VectorZero());
			}

			MaxNormalizedSize_VisibleOnly = VectorMax(ScreenSizeOverDistance, MaxNormalizedSize_VisibleOnly);
		}

		// Store results
		FBoundsViewInfo* BoundsVieWInfo = &OutBoundsViewInfos[Bounds4Index * 4];
		MS_ALIGN(16) float MaxNormalizedSizeScalar[4] GCC_ALIGN(16);
		VectorStoreAligned(MaxNormalizedSize, MaxNormalizedSizeScalar);
		MS_ALIGN(16) float MaxNormalizedSize_VisibleOnlyScalar[4] GCC_ALIGN(16);
		VectorStoreAligned(MaxNormalizedSize_VisibleOnly, MaxNormalizedSize_VisibleOnlyScalar);
		MS_ALIGN(16) float ComponentScaleScalar[4] GCC_ALIGN(16);
		VectorStoreAligned(ComponentScale, ComponentScaleScalar);
		for (int32 SubIndex = 0; SubIndex < 4; ++SubIndex)
		{
			BoundsVieWInfo[SubIndex].MaxNormalizedSize = MaxNormalizedSizeScalar[SubIndex];
			BoundsVieWInfo[SubIndex].MaxNormalizedSize_VisibleOnly = MaxNormalizedSize_VisibleOnlyScalar[SubIndex];
			BoundsVieWInfo[SubIndex].ComponentScale = ComponentScaleScalar[SubIndex];
		}
	}

	if (Settings.MinLevelRenderAssetScreenSize > 0)
	{
		float ViewMaxNormalizedSizeResult = VectorGetComponent(ViewMaxNormalizedSize, 0);
		MS_ALIGN(16) float ViewMaxNormalizedSizeScalar[4] GCC_ALIGN(16);
		VectorStoreAligned(ViewMaxNormalizedSize, ViewMaxNormalizedSizeScalar);
		for (int32 SubIndex = 1; SubIndex < 4; ++SubIndex)
		{
			ViewMaxNormalizedSizeResult = FMath::Max(ViewMaxNormalizedSizeResult, ViewMaxNormalizedSizeScalar[SubIndex]);
		}
		OutMaxLevelRenderAssetScreenSize = MaxTexelFactor * ViewMaxNormalizedSizeResult;
	}
}

void FRenderAssetInstanceAsyncView::ProcessElement(
	EStreamableRenderAssetType AssetType,
	const FBoundsViewInfo& BoundsVieWInfo,
	float TexelFactor,
	bool bForceLoad,
	float& MaxSize,
	float& MaxSize_VisibleOnly,
	int32& MaxNumForcedLODs)
{
	if (TexelFactor == FLT_MAX) // If this is a forced load component.
	{
		MaxSize = BoundsVieWInfo.MaxNormalizedSize > 0 ? FLT_MAX : MaxSize;
		MaxSize_VisibleOnly = BoundsVieWInfo.MaxNormalizedSize_VisibleOnly > 0 ? FLT_MAX : MaxSize_VisibleOnly;
	}
	else if (TexelFactor >= 0)
	{
		MaxSize = FMath::Max(MaxSize, TexelFactor * BoundsVieWInfo.MaxNormalizedSize);
		MaxSize_VisibleOnly = FMath::Max(MaxSize_VisibleOnly, TexelFactor * BoundsVieWInfo.MaxNormalizedSize_VisibleOnly);

		// Force load will load the immediately visible part, and later the full texture.
		if (bForceLoad && (BoundsVieWInfo.MaxNormalizedSize > 0 || BoundsVieWInfo.MaxNormalizedSize_VisibleOnly > 0))
		{
			MaxSize = FLT_MAX;
		}
	}
	else // Negative texel factors map to fixed resolution. Currently used for lanscape.
	{
		if (AssetType == EStreamableRenderAssetType::Texture)
		{
			MaxSize = FMath::Max(MaxSize, -TexelFactor);
			MaxSize_VisibleOnly = FMath::Max(MaxSize_VisibleOnly, -TexelFactor);
		}
		else
		{
			check(AssetType == EStreamableRenderAssetType::StaticMesh || AssetType == EStreamableRenderAssetType::SkeletalMesh);
			check(-TexelFactor <= (float)MAX_MESH_LOD_COUNT);
			MaxNumForcedLODs = FMath::Max(MaxNumForcedLODs, static_cast<int32>(-TexelFactor));
		}

		// Force load will load the immediatly visible part, and later the full texture.
		if (bForceLoad && (BoundsVieWInfo.MaxNormalizedSize > 0 || BoundsVieWInfo.MaxNormalizedSize_VisibleOnly > 0))
		{
			MaxSize = FLT_MAX;
			MaxSize_VisibleOnly = FLT_MAX;
		}
	}
}

void FRenderAssetInstanceAsyncView::GetRenderAssetScreenSize(
	EStreamableRenderAssetType AssetType,
	const UStreamableRenderAsset* InAsset,
	float& MaxSize,
	float& MaxSize_VisibleOnly,
	int32& MaxNumForcedLODs,
	const float MaxAssetSize,
	const TCHAR* LogPrefix) const
{
	// No need to iterate more if texture is already at maximum resolution.
	// Meshes don't really fit into the concept of max resolution but current
	// max_res for texture is 8k which is large enough to let mesh screen
	// sizes be constrained by this value

	int32 CurrCount = 0;

	if (View.IsValid())
	{
		// Use the fast path if available, about twice as fast when there are a lot of elements.
		if (View->HasCompiledElements() && !LogPrefix)
		{
			const TArray<FRenderAssetInstanceView::FCompiledElement>* CompiledElements = View->GetCompiledElements(InAsset);
			if (CompiledElements)
			{
				const int32 NumCompiledElements = CompiledElements->Num();
				const FRenderAssetInstanceView::FCompiledElement* CompiledElementData = CompiledElements->GetData();

				int32 CompiledElementIndex = 0;
				while (CompiledElementIndex < NumCompiledElements && MaxSize_VisibleOnly < MaxAssetSize)
				{
					const FRenderAssetInstanceView::FCompiledElement& CompiledElement = CompiledElementData[CompiledElementIndex];
					if (ensure(BoundsViewInfo.IsValidIndex(CompiledElement.BoundsIndex)))
					{
						// Texel factor wasn't available because the component wasn't registered. Lazy initialize it now.
						if (AssetType != EStreamableRenderAssetType::Texture
							&& CompiledElement.TexelFactor == 0.f
							&& ensure(CompiledElement.BoundsIndex < View->NumBounds4() * 4))
						{
							FRenderAssetInstanceView::FCompiledElement* MutableCompiledElement = const_cast<FRenderAssetInstanceView::FCompiledElement*>(&CompiledElement);
							MutableCompiledElement->TexelFactor = View->GetBounds4(CompiledElement.BoundsIndex / 4).RadiusOrComponentScale.Component(CompiledElement.BoundsIndex % 4) * 2.f;
						}

						ProcessElement(
							AssetType,
							BoundsViewInfo[CompiledElement.BoundsIndex],
							CompiledElement.TexelFactor,
							CompiledElement.bForceLoad,
							MaxSize,
							MaxSize_VisibleOnly,
							MaxNumForcedLODs);
					}
					++CompiledElementIndex;
				}

				if (MaxSize_VisibleOnly >= MaxAssetSize && CompiledElementIndex > 1)
				{
					// This does not realloc anything but moves the closest element at head, making the next update find it immediately and early exit.
					FRenderAssetInstanceView::FCompiledElement* SwapElementData = const_cast<FRenderAssetInstanceView::FCompiledElement*>(CompiledElementData);
					Swap<FRenderAssetInstanceView::FCompiledElement>(SwapElementData[0], SwapElementData[CompiledElementIndex - 1]);
				}
			}
		}
		else
		{
			for (auto It = View->GetElementIterator(InAsset); It && (AssetType != EStreamableRenderAssetType::Texture || MaxSize_VisibleOnly < MaxAssetSize || LogPrefix); ++It)
			{
				// Only handle elements that are in bounds.
				if (ensure(BoundsViewInfo.IsValidIndex(It.GetBoundsIndex())))
				{
					const FBoundsViewInfo& BoundsVieWInfo = BoundsViewInfo[It.GetBoundsIndex()];
					ProcessElement(AssetType, BoundsVieWInfo, AssetType != EStreamableRenderAssetType::Texture ? It.GetTexelFactor() : It.GetTexelFactor() * BoundsVieWInfo.ComponentScale, It.GetForceLoad(), MaxSize, MaxSize_VisibleOnly, MaxNumForcedLODs);
					if (LogPrefix)
					{
						It.OutputToLog(BoundsVieWInfo.MaxNormalizedSize, BoundsVieWInfo.MaxNormalizedSize_VisibleOnly, LogPrefix);
					}
				}
			}
		}
	}
}
