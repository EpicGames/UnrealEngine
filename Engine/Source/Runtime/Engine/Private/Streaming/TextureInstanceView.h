// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
TextureInstanceView.h: Definitions of classes used for texture streaming.
=============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "Templates/RefCounting.h"
#include "Containers/ChunkedArray.h"
#include "Streaming/StreamingTexture.h"

class UPrimitiveComponent;
class UStreamableRenderAsset;
struct FStreamingViewInfo;
struct FRenderAssetStreamingSettings;

#define MAX_TEXTURE_SIZE (float(1 << (MAX_TEXTURE_MIP_COUNT - 1)))


// The bounds are into their own arrays to allow SIMD-friendly processing
struct FBounds4
{
	FORCEINLINE FBounds4();

	/** X coordinates for the bounds origin of 4 texture/mesh instances */
	FVector4 OriginX;
	/** Y coordinates for the bounds origin of 4 texture/mesh instances */
	FVector4 OriginY;
	/** Z coordinates for the bounds origin of 4 texture/mesh instances */
	FVector4 OriginZ;

	/** X coordinates for used to compute the distance condition between min and max */
	FVector4 RangeOriginX;
	/** Y coordinates for used to compute the distance condition between min and max */
	FVector4 RangeOriginY;
	/** Z coordinates for used to compute the distance condition between min and max */
	FVector4 RangeOriginZ;

	/** X size of the bounds box extent of 4 texture/mesh instances */
	FVector4f ExtentX;
	/** Y size of the bounds box extent of 4 texture/mesh instances */
	FVector4f ExtentY;
	/** Z size of the bounds box extent of 4 texture/mesh instances */
	FVector4f ExtentZ;

	/** Sphere radii for the bounding sphere of 4 texture/mesh static instances or component scale for dynamic instances.
	 *  When the bounds row is owned by FSimpleStreamableAssetManager (SSAM), this field always carries
	 *  StreamingScaleFactor (i.e., the ComponentScale interpretation) regardless of whether the instance is
	 *  static or dynamic; the legacy "sphere radius for static" interpretation does not apply on SSAM-owned
	 *  rows. Consumers that branch on static vs dynamic must account for the SSAM ownership case. */
	FVector4f RadiusOrComponentScale;

	/** The relative box the bound was computed with. Aligned to be interpreted as FVector4  */
	MS_ALIGN(16) FUintVector4 PackedRelativeBox;

	/** Minimal distance (between the bounding sphere origin and the view origin) for which this entry is valid */
	FVector4f MinDistanceSq;
	/** Minimal range distance (between the bounding sphere origin and the view origin) for which this entry is valid */
	FVector4f MinRangeSq;
	/** Maximal range distance (between the bounding sphere origin and the view origin) for which this entry is valid */
	FVector4f MaxRangeSq;

	/** Last visibility time for this bound, used for priority */
	FVector4f LastRenderTime; //(FApp::GetCurrentTime() - Component->LastRenderTime);


	void Set(int32 Index, const FBoxSphereBounds& InBounds, uint32 InPackedRelativeBox, float InLastRenderTime, const FVector& InRangeOrigin, float InMinDistanceSq, float InMinRangeSq, float InMaxRangeSq);
	void Set(int32 Index, const FBoxSphereBounds& InBounds, uint32 InPackedRelativeBox, float InRadiusOrComponentScale, float InLastRenderTime, const FVector& InRangeOrigin, float InMinDistanceSq, float InMinRangeSq, float InMaxRangeSq);
	void UnpackBounds(int32 Index, const IPrimitiveComponent* Component);
	void FullUpdate(int32 Index, const FVector& NewOrigin, const FVector& NewBoxExtent, float NewSphereRadius, float LastRenderTime);
	FORCEINLINE void UpdateLastRenderTime(int32 Index, float LastRenderTime);
	FORCEINLINE void UpdateMaxDrawDistanceSquared(int32 Index, float InMaxRangeSq);

	// Clears entry between 0 and 4
	FORCEINLINE void Clear(int32 Index);

	FORCEINLINE void OffsetBounds(int32 Index, const FVector& Offset);
};

struct FBoundsViewInfo
{
	/** The biggest normalized size (ScreenSize / Distance) accross all view.*/
	float MaxNormalizedSize;
	/*
	 * The biggest normalized size accross all view for visible instances only.
	 * Visible instances are the one that are in range and also that have been seen recently.
	 */
	float MaxNormalizedSize_VisibleOnly;
	/** A custom per component scale applied to the texel factor. */
	float ComponentScale;
};

struct FStreamingViewInfoExtra
{
	// The screen size factor including the view boost.
	float ScreenSizeFloat;

	// The extra view boost for visible primitive (if ViewInfo.BoostFactor > "r.Streaming.MaxHiddenPrimitiveViewBoost").
	float ExtraBoostForVisiblePrimitiveFloat;
};

typedef TArray<FStreamingViewInfoExtra, TInlineAllocator<4>> FStreamingViewInfoExtraArray;

// Main Thread Job Requirement : find all instance of a component and update it's bound.
// Threaded Job Requirement : get the list of instance texture or mesh easily from the list of 

// A constant view on the relationship between textures/meshes, components and bounds. 
// Has everything needed for the worker task to compute the required size per texture/mesh.

class FRenderAssetInstanceView : public UE::Private::FQueryableRefCountedObject
{
public:

	struct FElement
	{
		FORCEINLINE FElement()
			: Component(nullptr)
			, RenderAsset(nullptr)
			, BoundsIndex(INDEX_NONE)
			, TexelFactor(0)
			, bForceLoad(false)
			, PrevRenderAssetLink(INDEX_NONE)
			, NextRenderAssetLink(INDEX_NONE)
			, NextComponentLink(INDEX_NONE)
		{
		}

		const IPrimitiveComponent* Component; // Which component this relates too
		const UStreamableRenderAsset* RenderAsset;	// Texture or mesh, never dereferenced.

		int32 BoundsIndex;		// The Index associated to this component (static component can have several bounds).
		float TexelFactor;		// The texture scale to be applied to this instance.
		bool bForceLoad;		// The texture or mesh needs to be force loaded.

		int32 PrevRenderAssetLink;	// The previous element which uses the same asset as this Element. The first element referred by RenderAssetMap will have INDEX_NONE.
		int32 NextRenderAssetLink;	// The next element which uses the same asset as this Element. Last element will have INDEX_NONE

		// Components are always updated as a whole, so individual elements can not be removed. Removing the need for PrevComponentLink.
		int32 NextComponentLink;	// The next element that uses the same component as this Element. The first one is referred by ComponentMap and the last one will have INDEX_NONE.
	};

	/**
 	 * FCompiledElement is a stripped down version of element and is stored in an array instead of using a linked list.
     * It is only used when the data is not expected to change and reduce that cache cost of iterating on all elements.
     **/
	struct FCompiledElement
	{
		FCompiledElement() {}
		FCompiledElement(const FElement& InElement) : BoundsIndex(InElement.BoundsIndex), TexelFactor(InElement.TexelFactor), bForceLoad(InElement.bForceLoad) {}

		int32 BoundsIndex;
		float TexelFactor;
		bool bForceLoad;

		FORCEINLINE bool operator==(const FCompiledElement& Rhs) const { return BoundsIndex == Rhs.BoundsIndex && TexelFactor == Rhs.TexelFactor && bForceLoad == Rhs.bForceLoad; }
	};

	struct FRenderAssetDesc
	{
		FRenderAssetDesc(int32 InHeadLink, int32 InLODGroup) : HeadLink(InHeadLink), LODGroup(InLODGroup) {}

		// The index of head element using the renderable asset.
		int32 HeadLink;
		// The LODGroup of the texture or mesh, used to performe some tasks async.
		const int32 LODGroup;
	};


	// Iterator processing all elements refering to a texture/mesh.
	class FRenderAssetLinkConstIterator
	{
	public:
		FRenderAssetLinkConstIterator(const FRenderAssetInstanceView& InState, const UStreamableRenderAsset* InAsset);

		FORCEINLINE explicit operator bool() const { return CurrElementIndex != INDEX_NONE; }
		FORCEINLINE void operator++() { CurrElementIndex = State.Elements[CurrElementIndex].NextRenderAssetLink; }

		void OutputToLog(float MaxNormalizedSize, float MaxNormalizedSize_VisibleOnly, const TCHAR* Prefix) const;

		FORCEINLINE int32 GetBoundsIndex() const { return State.Elements[CurrElementIndex].BoundsIndex; }
		FORCEINLINE float GetTexelFactor() const { return State.Elements[CurrElementIndex].TexelFactor; }
		FORCEINLINE bool GetForceLoad() const { return State.Elements[CurrElementIndex].bForceLoad; }

		FBoxSphereBounds GetBounds() const;

		FORCEINLINE const IPrimitiveComponent* GetComponent() const { return State.Elements[CurrElementIndex].Component; }

		const FRenderAssetInstanceView& State;
		int32 CurrElementIndex;
	};

	class FRenderAssetLinkIterator : public FRenderAssetLinkConstIterator
	{
	public:
		FRenderAssetLinkIterator(FRenderAssetInstanceView& InState, const UStreamableRenderAsset* InAsset) : FRenderAssetLinkConstIterator(InState, InAsset) {}

		FORCEINLINE void ClampTexelFactor(float CMin, float CMax)
		{ 
			float& TexelFactor = const_cast<FRenderAssetInstanceView&>(State).Elements[CurrElementIndex].TexelFactor;
			TexelFactor = FMath::Clamp<float>(TexelFactor, CMin, CMax);
		}
	};

	class FRenderAssetIterator
	{
	public:
		FRenderAssetIterator(const FRenderAssetInstanceView& InState) : MapIt(InState.RenderAssetMap) {}

		explicit operator bool() const { return (bool)MapIt; }
		void operator++() { ++MapIt; }

		const UStreamableRenderAsset* operator*() const { return MapIt.Key(); }
		int32 GetLODGroup() const { return MapIt.Value().LODGroup; }

	private:

		TMap<const UStreamableRenderAsset*, FRenderAssetDesc>::TConstIterator MapIt;
	};

	FRenderAssetInstanceView() : MaxTexelFactor(FLT_MAX) {}

	FORCEINLINE int32 NumBounds4() const { return Bounds4.Num(); }
	FORCEINLINE const FBounds4& GetBounds4(int32 Bounds4Index ) const {  return Bounds4[Bounds4Index]; }
	FORCEINLINE TConstArrayView<FBounds4> GetBoundsView() const {  return Bounds4; }

	FORCEINLINE FRenderAssetLinkIterator GetElementIterator(const UStreamableRenderAsset* InTexture ) {  return FRenderAssetLinkIterator(*this, InTexture); }
	FORCEINLINE FRenderAssetLinkConstIterator GetElementIterator(const UStreamableRenderAsset* InTexture ) const {  return FRenderAssetLinkConstIterator(*this, InTexture); }
	FORCEINLINE FRenderAssetIterator GetRenderAssetIterator( ) const {  return FRenderAssetIterator(*this); }

	// Whether or not this state has compiled elements.
	bool HasCompiledElements() const { return CompiledRenderAssetMap.Num() != 0; }
	// If this has compiled elements, return the array relate to a given texture or mesh.
	const TArray<FCompiledElement>* GetCompiledElements(const UStreamableRenderAsset* Asset) const { return CompiledRenderAssetMap.Find(Asset); }

	bool HasComponentWithForcedLOD(const UStreamableRenderAsset* Asset) const { return !!CompiledNumForcedLODCompMap.Find(Asset); }
	bool HasAnyComponentWithForcedLOD() const { return !!CompiledNumForcedLODCompMap.Num(); }

	static TRefCountPtr<const FRenderAssetInstanceView> CreateView(const FRenderAssetInstanceView* RefView);
	static TRefCountPtr<FRenderAssetInstanceView> CreateViewWithUninitializedBounds(const FRenderAssetInstanceView* RefView);
	static void SwapData(FRenderAssetInstanceView* Lfs, FRenderAssetInstanceView* Rhs);

	float GetMaxTexelFactor() const { return MaxTexelFactor; }

	static float GetMaxDrawDistSqWithLODParent(const FVector& Origin, const FVector& ParentOrigin, float ParentMinDrawDist, float ParentBoundingSphereRadius);

	static void GetDistanceAndRange(const IPrimitiveComponent* Component, const FBoxSphereBounds& RenderAssetInstanceBounds, float& MinDistanceSq, float& MinRangeSq, float& MaxRangeSq);

	static void OutputToLog(const FBoxSphereBounds& Bounds, int32 BoundsIndex, const TArray<FBounds4>& Bounds4, float TexelFactor, bool bForceLoad, float MaxNormalizedSize, float MaxNormalizedSize_VisibleOnly, const TCHAR* Prefix, const IPrimitiveComponent* Component = nullptr);

protected:

	TArray<FBounds4> Bounds4;

	TChunkedArray<FElement> Elements;

	TMap<const UStreamableRenderAsset*, FRenderAssetDesc> RenderAssetMap;

	// CompiledTextureMap is used to iterate more quickly on each elements by avoiding the linked list indirections.
	TMap<const UStreamableRenderAsset*, TArray<FCompiledElement> > CompiledRenderAssetMap;

	// If an asset has components with forced LOD levels, it will appear here after level compilation.
	TMap<const UStreamableRenderAsset*, int32> CompiledNumForcedLODCompMap;

	/** Max texel factor across all elements. Used for early culling */
	float MaxTexelFactor;
};

// Data used to compute visibility
class FRenderAssetInstanceAsyncView
{
public:

	FRenderAssetInstanceAsyncView() : MaxLevelRenderAssetScreenSize(UE_MAX_FLT) {}

	FRenderAssetInstanceAsyncView(const FRenderAssetInstanceView* InView) : View(InView), MaxLevelRenderAssetScreenSize(UE_MAX_FLT) {}

	void UpdateBoundSizes_Async(
		const TArray<FStreamingViewInfo>& ViewInfos,
		const FStreamingViewInfoExtraArray& ViewInfoExtras,
		float LastUpdateTime,
		const FRenderAssetStreamingSettings& Settings);

	// Screen size is the number of screen pixels overlap with this asset
	// MaxSize : Biggest screen size for all instances.
	// MaxSize_VisibleOnly : Biggest screen size for visble instances only.
	// MaxForcedNumLODs: max number of LODs forced resident. Used for meshes only
	void GetRenderAssetScreenSize(
		EStreamableRenderAssetType AssetType,
		const UStreamableRenderAsset* InAsset,
		float& MaxSize,
		float& MaxSize_VisibleOnly,
		int32& MaxNumForcedLODs,
		const float MaxAssetSize,
		const TCHAR* LogPrefix) const;

	FORCEINLINE bool HasRenderAssetReferences(const UStreamableRenderAsset* InAsset) const
	{
		return View.IsValid() && (bool)View->GetElementIterator(InAsset);
	}

	FORCEINLINE bool HasComponentWithForcedLOD(const UStreamableRenderAsset* InAsset) const
	{
		return View.IsValid() && View->HasComponentWithForcedLOD(InAsset);
	}

	FORCEINLINE bool HasAnyComponentWithForcedLOD() const
	{
		return View.IsValid() && View->HasAnyComponentWithForcedLOD();
	}

	// Release the data now as this is expensive.
	void OnTaskDone() { BoundsViewInfo.Empty(); }

	float GetMaxLevelRenderAssetScreenSize() const { return MaxLevelRenderAssetScreenSize; }

private:

	TRefCountPtr<const FRenderAssetInstanceView> View;

	// Normalized Texel Factors for each bounds and view. This is the data built by ComputeBoundsViewInfos
	// @TODO : store data for different views continuously to improve reads.
	TArray<FBoundsViewInfo> BoundsViewInfo;

	/** The max possible size (conservative) across all elements of this view. */
	float MaxLevelRenderAssetScreenSize;

public:
	static void UpdateBoundSizes(
		const TArray<FStreamingViewInfo>& ViewInfos,
		const FStreamingViewInfoExtraArray& ViewInfoExtras,
		float LastUpdateTime,
		float MaxTexelFactor,
		const FRenderAssetStreamingSettings& Settings,
		const TConstArrayView<FBounds4>& Bounds,
		TArray<FBoundsViewInfo>& OutBoundsViewInfos,
		float& OutMaxLevelRenderAssetScreenSize
		);

	static void ProcessElement(
		EStreamableRenderAssetType AssetType,
		const FBoundsViewInfo& BoundsVieWInfo,
		float TexelFactor,
		bool bForceLoad,
		float& MaxSize,
		float& MaxSize_VisibleOnly,
		int32& MaxNumForcedLODs);
};
