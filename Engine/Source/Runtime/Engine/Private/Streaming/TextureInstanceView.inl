// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
TextureInstanceView.h: Definitions of classes used for texture streaming.
=============================================================================*/

#pragma once

FBounds4::FBounds4()
:	OriginX( 0, 0, 0, 0 )
,	OriginY( 0, 0, 0, 0 )
,	OriginZ( 0, 0, 0, 0 )
,	RangeOriginX( 0, 0, 0, 0 )
,	RangeOriginY( 0, 0, 0, 0 )
,	RangeOriginZ( 0, 0, 0, 0 )
,	ExtentX( 0, 0, 0, 0 )
,	ExtentY( 0, 0, 0, 0 )
,	ExtentZ( 0, 0, 0, 0 )
,	RadiusOrComponentScale( 0, 0, 0, 0 )
,	PackedRelativeBox(0, 0, 0, 0)
,	MinDistanceSq( 0, 0, 0, 0 )
,	MinRangeSq( 0, 0, 0, 0 )
,	MaxRangeSq( FLT_MAX, FLT_MAX, FLT_MAX, FLT_MAX )
,	LastRenderTime( -FLT_MAX, -FLT_MAX, -FLT_MAX, -FLT_MAX)
{
}

void FBounds4::Clear(int32 Index)
{
	check(Index >= 0 && Index < 4);

	OriginX.Component(Index) = 0;
	OriginY.Component(Index) = 0;
	OriginZ.Component(Index) = 0;
	RangeOriginX.Component(Index) = 0;
	RangeOriginY.Component(Index) = 0;
	RangeOriginZ.Component(Index) = 0;
	ExtentX.Component(Index) = 0;
	ExtentY.Component(Index) = 0;
	ExtentZ.Component(Index) = 0;
	RadiusOrComponentScale.Component(Index) = 0;
	PackedRelativeBox[Index] = 0;
	MinDistanceSq.Component(Index) = 0; 
	MinRangeSq.Component(Index) = 0;
	MaxRangeSq.Component(Index) = FLT_MAX;
	LastRenderTime.Component(Index) = -FLT_MAX;
}

void FBounds4::OffsetBounds(int32 Index, const FVector& Offset)
{
	check(Index >= 0 && Index < 4);

	OriginX.Component(Index) += Offset.X;
	OriginY.Component(Index) += Offset.Y;
	OriginZ.Component(Index) += Offset.Z;
}

void FBounds4::UpdateLastRenderTime(int32 Index, float InLastRenderTime)
{
	check(Index >= 0 && Index < 4);

	LastRenderTime.Component(Index) = InLastRenderTime;
}

void FBounds4::UpdateMaxDrawDistanceSquared(int32 Index, float InMaxRangeSq)
{
	check(Index >= 0 && Index < 4);
	MaxRangeSq.Component(Index) = InMaxRangeSq;
}
