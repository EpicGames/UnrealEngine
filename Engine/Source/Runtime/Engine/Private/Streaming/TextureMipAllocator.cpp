// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
TextureMipAllocator.cpp: Base class for implementing a mip allocation strategy used by FTextureStreamIn.
=============================================================================*/

#include "Streaming/TextureMipAllocator.h"
#include "Streaming/TextureMipDataProvider.h"
#include "Rendering/StreamableTextureResource.h"
#include "Engine/Texture.h"

FTextureMipAllocator::FTextureMipAllocator(UTexture* Texture, ETickState InTickState, ETickThread InTickThread)
	: ResourceState(Texture->GetStreamableResourceState())
	, CurrentFirstLODIdx(Texture->GetStreamableResourceState().ResidentFirstLODIdx())
	, PendingFirstLODIdx(Texture->GetStreamableResourceState().RequestedFirstLODIdx())
	, NextTickState(InTickState)
	, NextTickThread(InTickThread) 
{
}

FTextureMipAllocator::FTextureMipAllocator(const FTextureMipAllocator&) = default;
FTextureMipAllocator::FTextureMipAllocator(FTextureMipAllocator&&) = default;
FTextureMipAllocator::~FTextureMipAllocator() = default;

bool FTextureMipAllocator::FinalizeMips(const FTextureUpdateContext& Context, const FTextureUpdateSyncOptions& SyncOptions)
{
	if (!ensureMsgf(IntermediateTextureRHI.IsValid(), TEXT("FTextureMipAllocator::FinalizeMips - Finalized called but a IntermediateTextureRHI is empty. This must be a valid texture reference before finalization")))
	{
		return false;
	}

	// Use the new texture resource for the texture asset, must run on the renderthread.
	Context.Resource->FinalizeStreaming(FRHICommandListImmediate::Get(), IntermediateTextureRHI);

	// No need for the intermediate texture anymore.
	IntermediateTextureRHI.SafeRelease();
	return true;
}

FTextureMipAllocator::ETickThread FTextureMipAllocator::ExecuteGetCancelThread() const
{
	if (IntermediateTextureRHI.IsValid())
	{
		return ETickThread::Render;
	}

	return GetCancelThread();
}

void FTextureMipAllocator::ExecuteCancel(const FTextureUpdateSyncOptions& SyncOptions)
{
	Cancel(SyncOptions);
	IntermediateTextureRHI.SafeRelease();
}
