// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
TextureMipAllocator.h: Base class for implementing a mip allocation strategy used by FTextureStreamIn.
=============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "RHIFwd.h"
#include "Streaming/TextureMipDataProvider.h"

/**
* FTextureMipAllocator defines the update steps and interface to implement the mip allocation strategy
* used in FTextureStreamIn. It allows to decouples the texture creation steps from the texture update.
* Typical implementations are using RHIAsyncCreateTexture2D, AsyncReallocateTexture2D or virtual textures.
*/
class FTextureMipAllocator
{
public:

	enum class ETickState : uint32
	{
		AllocateMips,	// Call AllocateMips(), which must provide destination memory for the mips to be written to.
		UploadMips,		// Call UploadMips(), which must release temporary buffers and update/upload the texture to the gpu.
		Done			// Nothing to do.
	};

	enum class ETickThread : uint32
	{
		Async,			// Tick must run on an async thread.
		Render,			// Tick must run on the renderthread.
		None,			// No tick must run, only valid in ETickState::Done. 
	};

	// Constructor, defining the first tick step and thread.
	FTextureMipAllocator(UTexture* Texture, ETickState InTickState, ETickThread InTickThread);
	FTextureMipAllocator(const FTextureMipAllocator&);
	FTextureMipAllocator(FTextureMipAllocator&&);
	virtual ~FTextureMipAllocator();

	// Get the next tick state and thread for the mip allocator. Used by FTextureStreamIn to schedule correctly the update between FTextureMipAllocator and FTextureMipDataProvider.
	FORCEINLINE ETickState GetNextTickState() const { return NextTickState; }
	FORCEINLINE ETickThread GetNextTickThread() const { return NextTickThread; }

	/**
	* Allocate the FTextureMipInfoArray and configure the entries for the new (streamed in) mips. 
	* In particular, the FTextureMipAllocator must set valid FTextureMipInfo::DestData so that the mip data provider can write the mip data to.
	*
	* @param Context - An update context constant throughout the FTextureStreamIn update. Gives things like which texture asset is updated and what mips are streamed in.
	* @param OutMipInfos - The array of FTextureMipInfo that must be correctly configured so that FTextureMipDataProvider know what is being requested and where to write data to.
	* @param SyncOptions - Different sync options to control when the next tick of FTextureStreamIn can be scheduled.
	*
	* Return true unless the texture update needs to be aborted because of invalid data or setup.
	*/
	virtual bool AllocateMips(const FTextureUpdateContext& Context, FTextureMipInfoArray& OutMipInfos, const FTextureUpdateSyncOptions& SyncOptions) = 0;
	
	/**
	* Upload the new mip data to the GPU.  The
	*
	* @param Context - An update context constant throughout the FTextureStreamIn update. Gives things like which texture asset is updated and what mips are streamed in.
	* @param SyncOptions - Different sync options to control when the next tick of FTextureStreamIn can be scheduled.
	*
	* Return true unless the texture update needs to be aborted because of invalid data or setup.
	*/
	virtual bool UploadMips(const FTextureUpdateContext& Context, const FTextureUpdateSyncOptions& SyncOptions) = 0;

	/**
	* Update the texture resource with the new RHI texture. Update is completed after this step and can not be canceled afterward..
	*
	* @param Context - An update context constant throughout the FTextureStreamIn update. Gives things like which texture asset is updated and what mips are streamed in.
	* @param SyncOptions - Different sync options to control when the next tick of FTextureStreamIn can be scheduled.
	*
	* Return true unless the texture update needs to be aborted because of invalid data or setup.
	*/
	bool FinalizeMips(const FTextureUpdateContext& Context, const FTextureUpdateSyncOptions& SyncOptions);

	// Wrapper around virtual Cancel.
	void ExecuteCancel(const FTextureUpdateSyncOptions& SyncOptions);

	// Wrapper around virtual GetCancelThread.
	ETickThread ExecuteGetCancelThread() const;

	const FTextureRHIRef& GetIntermediateTextureRHI() const { return IntermediateTextureRHI; }

protected:

	/**
	* Cancel the progression and release any temporary resources.
	* Called within the FTextureStreamIn update when the stream in request is aborted or cannot complete correctly
	* The Cancel() function is called on the thread returned by GetCancelThread().
	*
	* @param SyncOptions - Different sync options to control when the next tick of FTextureStreamIn can be scheduled.
	*/
	virtual void Cancel(const FTextureUpdateSyncOptions& SyncOptions) = 0;

	// Returns on which thread Cancel() must be called in the texture update (from FTextureStreamIn) to release resources safely and correctly. ETickThread::None when ready to delete this.
	virtual ETickThread GetCancelThread() const = 0;

	// Helper to set the next tick state and thread. Validates that the progression is coherent.
	FORCEINLINE void AdvanceTo(ETickState InState, ETickThread InThread)
	{
		// State must advance or at least stay to the same step.
		check((uint32)NextTickState <= (uint32)InState);
		// Thread must be valid unless in done state.
		check((InState == ETickState::Done) == (InThread == ETickThread::None));

		NextTickState = InState;
		NextTickThread = InThread;
	}

	/** The streamable state requested. */
	const FStreamableRenderResourceState ResourceState;
	// The resident first LOD resource index. With domain = [0, ResourceState.NumLODs[. NOT THE ASSET LOD INDEX!
	const int32 CurrentFirstLODIdx = INDEX_NONE;
	// The requested first LOD resource index. With domain = [0, ResourceState.NumLODs[. NOT THE ASSET LOD INDEX!
	const int32 PendingFirstLODIdx = INDEX_NONE;

	/**  The intermediate texture async created in the update process. */
	FTextureRHIRef IntermediateTextureRHI;

private:

	// The next tick function that should be called.
	ETickState NextTickState = ETickState::Done;
	// The next tick thread from where to call the tick function.
	ETickThread NextTickThread = ETickThread::None;
};
