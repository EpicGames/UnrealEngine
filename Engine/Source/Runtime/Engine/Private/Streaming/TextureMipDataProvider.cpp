// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
TextureMipDataProvider.cpp: Base class for providing the mip data used by FTextureStreamIn.
=============================================================================*/

#include "Streaming/TextureMipDataProvider.h"
#include "Rendering/StreamableTextureResource.h"
#include "Templates/Casts.h"

FTextureUpdateContext::FTextureUpdateContext(const UTexture* InTexture, EThreadType InCurrentThread) 
	: Texture(InTexture)
	, CurrentThread(InCurrentThread)
{
	check(InTexture);
	Resource = Texture && Texture->GetResource() ? const_cast<UTexture*>(Texture)->GetResource()->GetStreamableTextureResource() : nullptr;
	if (Resource)
	{
		MipsView = Resource->GetPlatformMipsView();
	}
}

FTextureUpdateContext::FTextureUpdateContext(const UStreamableRenderAsset* InTexture, EThreadType InCurrentThread)
	: FTextureUpdateContext(CastChecked<UTexture>(InTexture), InCurrentThread)
{
}

FTextureMipDataProvider::FTextureMipDataProvider(const UTexture* Texture, ETickState InTickState, ETickThread InTickThread)
	: ResourceState(Texture->GetStreamableResourceState())
	, CurrentFirstLODIdx(Texture->GetStreamableResourceState().ResidentFirstLODIdx())
	, PendingFirstLODIdx(Texture->GetStreamableResourceState().RequestedFirstLODIdx())
	, NextTickState(InTickState)
	, NextTickThread(InTickThread) 
{
}

bool FTextureMipDataProvider::PostprocessGPUMipData(const FTextureUpdateContext& Context, const FTextureRHIRef& TextureRHI, int32 FirstMip, int32 NumMips, const FTextureUpdateSyncOptions& SyncOptions)
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	return PostprocessGPUMipData(Context, TextureRHI, SyncOptions);
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}
