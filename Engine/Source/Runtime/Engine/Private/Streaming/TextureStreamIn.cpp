// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
FTextureStreamIn.cpp : Implement a generic texture stream in strategy.
=============================================================================*/

#include "Streaming/TextureStreamIn.h"
#include "Engine/Texture.h"
#include "Streaming/RenderAssetUpdate.inl"
#include "Streaming/TextureMipAllocator.h"

template class TRenderAssetUpdate<FTextureUpdateContext>;

bool FTextureStreamIn::IsSameThread(FTextureMipAllocator::ETickThread TickThread, int32 TaskThread)
{
	if (TaskThread == TT_Async)
	{
		return TickThread == FTextureMipAllocator::ETickThread::Async;
	}
	else // Expected to be called from either the async thread or the renderthread.
	{
		check(TaskThread == TT_Render);
		return TickThread == FTextureMipAllocator::ETickThread::Render;
	}
}

bool FTextureStreamIn::IsSameThread(FTextureMipDataProvider::ETickThread TickThread, int32 TaskThread)
{
	if (TaskThread == TT_Async)
	{
		return TickThread == FTextureMipDataProvider::ETickThread::Async;
	}
	else // Expected to be called from either the async thread or the renderthread.
	{
		check(TaskThread == TT_Render);
		return TickThread == FTextureMipDataProvider::ETickThread::Render;
	}
}

FTextureStreamIn::FTextureStreamIn(
	const UTexture* InTexture, 
	FTextureMipAllocator* InMipAllocator, 
	FTextureMipDataProvider* InCustomMipDataProvider, 
	FTextureMipDataProvider* InDefaultMipDataProvider
)
	: TRenderAssetUpdate<FTextureUpdateContext>(InTexture)
{
	check(InMipAllocator);
	StartingMipIndex = PendingFirstLODIdx;

	// Init the allocator and provider.
	MipAllocator.Reset(InMipAllocator);
	if (InCustomMipDataProvider)
	{
		MipDataProviders.Emplace(InCustomMipDataProvider);
	}
	check(InDefaultMipDataProvider);
	MipDataProviders.Emplace(InDefaultMipDataProvider);

	// Init sync options
	SyncOptions.bSnooze = &bDeferExecution;
	SyncOptions.Counter = &TaskSynchronization;
	SyncOptions.RescheduleCallback = [this]() 
	{ 
		if (!IsLocked())
		{
			Tick(TT_None); 
		}
	};

	// Schedule the first update step.
	FContext Context(InTexture, TT_None);
	const EThreadType NextThread = GetMipDataProviderThread(FTextureMipDataProvider::ETickState::Init);
	if (NextThread != TT_None)
	{
		PushTask(Context, NextThread, SRA_UPDATE_CALLBACK(InitMipDataProviders), TT_None, nullptr);
	}
	else // Otherwise nothing to execute but schedule next phase (edge case).
	{
		InitMipDataProviders(Context);
	}
}

FTextureStreamIn::~FTextureStreamIn()
{
}

FRenderAssetUpdate::EThreadType FTextureStreamIn::GetMipDataProviderThread(FTextureMipDataProvider::ETickState TickState) const
{
	if (!IsCancelled())
	{
		for (const TUniquePtr<FTextureMipDataProvider>& MipDataProvider : MipDataProviders)
		{
			check(MipDataProvider);
			if (MipDataProvider->GetNextTickState() == TickState)
			{
				switch (MipDataProvider->GetNextTickThread())
				{
				case FTextureMipDataProvider::ETickThread::Async:
					return TT_Async;
				case FTextureMipDataProvider::ETickThread::Render:
					return TT_Render;
				default:
					check(false);
				}
			}
		}
	}
	return TT_None;
}

FRenderAssetUpdate::EThreadType FTextureStreamIn::GetMipAllocatorThread(FTextureMipAllocator::ETickState TickState) const
{
	check(MipAllocator);
	if (!IsCancelled() && MipAllocator->GetNextTickState() == TickState)
	{
		switch (MipAllocator->GetNextTickThread())
		{
		case FTextureMipAllocator::ETickThread::Async:
			return TT_Async;
		case FTextureMipAllocator::ETickThread::Render:
			return TT_Render;
		default:
			check(false);
		}
	}
	return TT_None;
}

FRenderAssetUpdate::EThreadType FTextureStreamIn::GetCancelThread() const
{
	// The mip data provider must be cancelled before the mip allocator since it writes into the mip data.
	for (const TUniquePtr<FTextureMipDataProvider>& MipDataProvider : MipDataProviders)
	{
		check(MipDataProvider);
		switch (MipDataProvider->GetCancelThread())
		{
		case FTextureMipDataProvider::ETickThread::Async:
			return TT_Async;
		case FTextureMipDataProvider::ETickThread::Render:
			return TT_Render;
		}
	}

	// If the mip data provider is fully cancelled, then cancel the mip allocator.
	if (MipAllocator)
	{
		switch (MipAllocator->ExecuteGetCancelThread())
		{
		case FTextureMipAllocator::ETickThread::Async:
			return TT_Async;
		case FTextureMipAllocator::ETickThread::Render:
			return TT_Render;
		}
	}


	// If both are cancelled, then run the final cleanup on the async thread.
	return TT_Async;
}

// ****************************
// **** Update Steps Work *****
// ****************************

void FTextureStreamIn::DoInitMipDataProviders(const FContext& Context)
{
	for (TUniquePtr<FTextureMipDataProvider>& MipDataProvider : MipDataProviders)
	{
		check(MipDataProvider);
		if (MipDataProvider->GetNextTickState() == FTextureMipDataProvider::ETickState::Init
			&& IsSameThread(MipDataProvider->GetNextTickThread(), Context.CurrentThread))
		{
			MipDataProvider->Init(Context, SyncOptions);
		}
	}
}

bool FTextureStreamIn::DoAllocateNewMips(const FContext& Context)
{
	check(MipAllocator && IsSameThread(MipAllocator->GetNextTickThread(), Context.CurrentThread));
	return MipAllocator->AllocateMips(Context, MipInfos, SyncOptions);
}

void FTextureStreamIn::DoGetMipData(const FContext& Context)
{
	for (TUniquePtr<FTextureMipDataProvider>& MipDataProvider : MipDataProviders)
	{
		check(MipDataProvider);
		if (MipDataProvider->GetNextTickState() == FTextureMipDataProvider::ETickState::GetMips
			&& IsSameThread(MipDataProvider->GetNextTickThread(), Context.CurrentThread))
		{
			StartingMipIndex = MipDataProvider->GetMips(Context, StartingMipIndex, MipInfos, SyncOptions);
		}
	}
}

bool FTextureStreamIn::DoPollMipData(const FContext& Context)
{
	for (TUniquePtr<FTextureMipDataProvider>& MipDataProvider : MipDataProviders)
	{
		check(MipDataProvider);
		if (MipDataProvider->GetNextTickState() == FTextureMipDataProvider::ETickState::PollMips
			&& IsSameThread(MipDataProvider->GetNextTickThread(), Context.CurrentThread))
		{
			if (!MipDataProvider->PollMips(SyncOptions))
			{
				return false;
			}
		}
	}
	return true;
}

bool FTextureStreamIn::DoUploadNewMips(const FContext& Context)
{
	check(MipAllocator && IsSameThread(MipAllocator->GetNextTickThread(), Context.CurrentThread));
	return MipAllocator->UploadMips(Context, SyncOptions);
}

bool FTextureStreamIn::DoPostprocessGPUMipData(const FContext& Context)
{
	const int32 MipOffset = !!(MipAllocator->GetIntermediateTextureRHI()->GetFlags() & TexCreate_Virtual) ? 0 : PendingFirstLODIdx;
	const int32 NumMips = CurrentFirstLODIdx - PendingFirstLODIdx;

	for (TUniquePtr<FTextureMipDataProvider>& MipDataProvider : MipDataProviders)
	{
		check(MipDataProvider);
		if (MipDataProvider->GetNextTickState() == FTextureMipDataProvider::ETickState::PostprocessGPUMipData
			&& IsSameThread(MipDataProvider->GetNextTickThread(), Context.CurrentThread))
		{
			if (!MipDataProvider->PostprocessGPUMipData(Context, MipAllocator->GetIntermediateTextureRHI(), PendingFirstLODIdx - MipOffset, NumMips, SyncOptions))
			{
				return false;
			}
		}
	}
	return true;
}

bool FTextureStreamIn::DoFinalizeNewMips(const FContext& Context)
{
	check(MipAllocator && IsSameThread(FTextureMipAllocator::ETickThread::Render, Context.CurrentThread));
	return MipAllocator->FinalizeMips(Context, SyncOptions);
}

void FTextureStreamIn::DoCleanUpMipDataProviders(const FContext& Context)
{
	for (TUniquePtr<FTextureMipDataProvider>& MipDataProvider : MipDataProviders)
	{
		check(MipDataProvider);
		if (MipDataProvider->GetNextTickState() == FTextureMipDataProvider::ETickState::CleanUp
			&& IsSameThread(MipDataProvider->GetNextTickThread(), Context.CurrentThread))
		{
			MipDataProvider->CleanUp(SyncOptions);
		}
	}
}

// ****************************
// ******* Update Steps *******
// ****************************

void FTextureStreamIn::InitMipDataProviders(const FContext& Context)
{
	DECLARE_SCOPE_CYCLE_COUNTER(TEXT("FTextureStreamIn::InitMipDataProviders"), STAT_TextureStreamIn_InitMipDataProviders, STATGROUP_StreamingDetails);

	// Execute
	DoInitMipDataProviders(Context);

	// Schedule the next update step.
	EThreadType NextThread = GetMipDataProviderThread(FTextureMipDataProvider::ETickState::Init);
	if (NextThread != TT_None) // Loop on this state.
	{
		PushTask(Context, NextThread, SRA_UPDATE_CALLBACK(InitMipDataProviders), GetCancelThread(), SRA_UPDATE_CALLBACK(Cancel));
	}
	else
	{
		NextThread = GetMipAllocatorThread(FTextureMipAllocator::ETickState::AllocateMips);
		if (NextThread != TT_None)
		{
			PushTask(Context, NextThread, SRA_UPDATE_CALLBACK(AllocateNewMips), GetCancelThread(), SRA_UPDATE_CALLBACK(Cancel));
		}
		else // Otherwise if it is impossible to allocate the new mips, abort.
		{
			MarkAsCancelled();
			PushTask(Context, TT_None, nullptr, GetCancelThread(), SRA_UPDATE_CALLBACK(Cancel));
		}
	}
}

void FTextureStreamIn::AllocateNewMips(const FContext& Context)
{
	DECLARE_SCOPE_CYCLE_COUNTER(TEXT("FTextureStreamIn::AllocateNewMips"), STAT_TextureStreamIn_AllocateNewMips, STATGROUP_StreamingDetails);

	// Execute
	if (!DoAllocateNewMips(Context))
	{
		MarkAsCancelled();
	}

	// Schedule the next update step.
	EThreadType NextThread = GetMipAllocatorThread(FTextureMipAllocator::ETickState::AllocateMips);
	if (NextThread != TT_None) // Loop on this state.
	{
		PushTask(Context, NextThread, SRA_UPDATE_CALLBACK(AllocateNewMips), GetCancelThread(), SRA_UPDATE_CALLBACK(Cancel));
	}
	else
	{
		// Check that all mips have data available. If not, something went wrong in DoAllocateNewMips() and everything must be cancelled.
		for (int32 MipIndex = PendingFirstLODIdx; MipIndex < CurrentFirstLODIdx; ++MipIndex)
		{
			if (!MipInfos.IsValidIndex(MipIndex) || !MipInfos[MipIndex].DestData)
			{
				MarkAsCancelled();
			}
		}

		NextThread = GetMipDataProviderThread(FTextureMipDataProvider::ETickState::GetMips);
		if (NextThread != TT_None)
		{
			PushTask(Context, NextThread, SRA_UPDATE_CALLBACK(GetMipData), GetCancelThread(), SRA_UPDATE_CALLBACK(Cancel));
		}
		else // Otherwise if it is impossible the get the mip data abort.
		{
			MarkAsCancelled();
			PushTask(Context, TT_None, nullptr, GetCancelThread(), SRA_UPDATE_CALLBACK(Cancel));
		}
	}
}

void FTextureStreamIn::GetMipData(const FContext& Context)
{
	DECLARE_SCOPE_CYCLE_COUNTER(TEXT("FTextureStreamIn::GetMipData"), STAT_TextureStreamIn_GetMipData, STATGROUP_StreamingDetails);

	// Execute
	DoGetMipData(Context);

	// Schedule the next update step.
	EThreadType NextThread = GetMipDataProviderThread(FTextureMipDataProvider::ETickState::GetMips);
	if (NextThread != TT_None) // Loop on this state.
	{
		PushTask(Context, NextThread, SRA_UPDATE_CALLBACK(GetMipData), GetCancelThread(), SRA_UPDATE_CALLBACK(Cancel));
	}
	else
	{
		// All mips must have been updated correctly.
		if (StartingMipIndex != CurrentFirstLODIdx)
		{
			MarkAsCancelled();
		}

		if (!IsCancelled())
		{
			NextThread = GetMipDataProviderThread(FTextureMipDataProvider::ETickState::PollMips);
			// All mips must be handled before moving to next stage.
			if (NextThread != TT_None)
			{
				bIsPollingMipData = true;
				PushTask(Context, NextThread, SRA_UPDATE_CALLBACK(PollMipData), GetCancelThread(), SRA_UPDATE_CALLBACK(Cancel));
			}
			else // Otherwise nothing to execute but schedule next phase (edge case).
			{
				PollMipData(Context);
			}
		}
		else
		{
			PushTask(Context, TT_None, nullptr, GetCancelThread(), SRA_UPDATE_CALLBACK(Cancel));
		}
	}
}

void FTextureStreamIn::PollMipData(const FContext& Context)
{
	DECLARE_SCOPE_CYCLE_COUNTER(TEXT("FTextureStreamIn::PollMipData"), STAT_TextureStreamIn_PollMipData, STATGROUP_StreamingDetails);

	// Execute
	if (!DoPollMipData(Context))
	{
		MarkAsCancelled();
	}

	// Schedule the next update step.
	EThreadType NextThread = GetMipDataProviderThread(FTextureMipDataProvider::ETickState::PollMips);
	if (NextThread != TT_None) // Loop on this state.
	{
		PushTask(Context, NextThread, SRA_UPDATE_CALLBACK(PollMipData), GetCancelThread(), SRA_UPDATE_CALLBACK(Cancel));
	}
	else
	{
		bIsPollingMipData = false;
		bool bCancelled = IsCancelled();
		if (!bCancelled)
		{
			NextThread = GetMipAllocatorThread(FTextureMipAllocator::ETickState::UploadMips);
			// All mips must be handled before moving to next stage.
			if (NextThread != TT_None)
			{
				PushTask(Context, NextThread, SRA_UPDATE_CALLBACK(UploadNewMips), GetCancelThread(), SRA_UPDATE_CALLBACK(Cancel));
			}
			else // Allocator does not specificy an UploadMips thread. This is a required operation before finalizing the texture.
			{
				MarkAsCancelled();
				bCancelled = true;
			}
		}

		if(bCancelled)
		{
			PushTask(Context, TT_None, nullptr, GetCancelThread(), SRA_UPDATE_CALLBACK(Cancel));
		}
	}
}

void FTextureStreamIn::UploadNewMips(const FContext& Context)
{
	DECLARE_SCOPE_CYCLE_COUNTER(TEXT("FTextureStreamIn::UploadNewMips"), STAT_TextureStreamIn_UploadNewMips, STATGROUP_StreamingDetails);

	// Execute
	if (!DoUploadNewMips(Context))
	{
		MarkAsCancelled();
	}

	// Schedule the next update step.
	EThreadType NextThread = GetMipAllocatorThread(FTextureMipAllocator::ETickState::UploadMips);

	if (NextThread != TT_None) // Loop on this state.
	{
		PushTask(Context, NextThread, SRA_UPDATE_CALLBACK(UploadNewMips), GetCancelThread(), SRA_UPDATE_CALLBACK(Cancel));
	}
	else if (!IsCancelled())
	{
		
		NextThread = GetMipDataProviderThread(FTextureMipDataProvider::ETickState::PostprocessGPUMipData);
		// All mips must be handled before moving to next stage.
		if (NextThread != TT_None)
		{
			bIsPostprocessingGPUMipData = true;
			PushTask(Context, NextThread, SRA_UPDATE_CALLBACK(PostprocessGPUMipData), GetCancelThread(), SRA_UPDATE_CALLBACK(Cancel));
		}
		else // Otherwise nothing to execute but schedule next phase (edge case).
		{
			PostprocessGPUMipData(Context);
		}
	}
	else
	{
		PushTask(Context, TT_None, nullptr, GetCancelThread(), SRA_UPDATE_CALLBACK(Cancel));
	}
}

void FTextureStreamIn::PostprocessGPUMipData(const FContext& Context)
{
	DECLARE_SCOPE_CYCLE_COUNTER(TEXT("FTextureStreamIn::PostprocessGPUMipData"), STAT_TextureStreamIn_PostprocessGPUMipData, STATGROUP_StreamingDetails);

	// Execute
	if (!DoPostprocessGPUMipData(Context))
	{
		MarkAsCancelled();
	}

	// Schedule the next update step.
	EThreadType NextThread = GetMipDataProviderThread(FTextureMipDataProvider::ETickState::PostprocessGPUMipData);

	if (NextThread != TT_None) // Loop on this state.
	{
		PushTask(Context, NextThread, SRA_UPDATE_CALLBACK(PostprocessGPUMipData), GetCancelThread(), SRA_UPDATE_CALLBACK(Cancel));
	}
	else if (!IsCancelled())
	{
		bIsPostprocessingGPUMipData = false;
		// The finalize task should always fire on the render thread
		PushTask(Context, EThreadType::TT_Render, SRA_UPDATE_CALLBACK(FinalizeNewMips), GetCancelThread(), SRA_UPDATE_CALLBACK(Cancel));
	}
	else
	{
		PushTask(Context, TT_None, nullptr, GetCancelThread(), SRA_UPDATE_CALLBACK(Cancel));
	}
}

void FTextureStreamIn::FinalizeNewMips(const FContext& Context)
{
	DECLARE_SCOPE_CYCLE_COUNTER(TEXT("FTextureStreamIn::FinalizeNewMips"), STAT_TextureStreamIn_FinalizeNewMips, STATGROUP_StreamingDetails);

	// Execute
	if (!DoFinalizeNewMips(Context))
	{
		MarkAsCancelled();
	}

	// Schedule the next update step.
	if (!IsCancelled())
	{
		// Finalize new mips completed successfully.
		MarkAsSuccessfullyFinished();
		
		// Release the mip allocator.
		MipAllocator.Reset();
		MipInfos.Empty();

		EThreadType NextThread = GetMipDataProviderThread(FTextureMipDataProvider::ETickState::CleanUp);
		// All mips must be handled before moving to next stage.
		if (NextThread != TT_None)
		{
			PushTask(Context, NextThread, SRA_UPDATE_CALLBACK(CleanUpMipDataProviders), GetCancelThread(), SRA_UPDATE_CALLBACK(Cancel));
		}
		else // Otherwise nothing to execute but schedule next phase (edge case).
		{
			CleanUpMipDataProviders(Context);
		}
	}
	else
	{
		// Mark as successfully finished even if canceled to keep the CachedSRRState in sync with the RHI texture.
		if (MipAllocator->GetNextTickState() == FTextureMipAllocator::ETickState::Done)
		{
			MarkAsSuccessfullyFinished();
		}

		PushTask(Context, TT_None, nullptr, GetCancelThread(), SRA_UPDATE_CALLBACK(Cancel));
	}
}

void FTextureStreamIn::CleanUpMipDataProviders(const FContext& Context)
{
	DECLARE_SCOPE_CYCLE_COUNTER(TEXT("FTextureStreamIn::CleanUpMipDataProviders"), STAT_TextureStreamIn_CleanUpMipDataProviders, STATGROUP_StreamingDetails);

	// Execute
	DoCleanUpMipDataProviders(Context);

	// Schedule the next update step.
	EThreadType NextThread = GetMipDataProviderThread(FTextureMipDataProvider::ETickState::CleanUp);
	if (NextThread != TT_None) // Loop on this state.
	{
		PushTask(Context, NextThread, SRA_UPDATE_CALLBACK(CleanUpMipDataProviders), GetCancelThread(), SRA_UPDATE_CALLBACK(Cancel));
	}
	else
	{
		// Release the mip data providers.
		MipDataProviders.Empty();
	}
}

void FTextureStreamIn::Cancel(const FContext& Context)
{
	DECLARE_SCOPE_CYCLE_COUNTER(TEXT("FTextureStreamIn::Cancel"), STAT_TextureStreamIn_Cancel, STATGROUP_StreamingDetails);

	EThreadType NextThread = TT_None;

	// The mip data provider must be cancelled before the mip allocator since it writes into the mip data.
	for (TUniquePtr<FTextureMipDataProvider>& MipDataProvider : MipDataProviders)
	{
		check(MipDataProvider);
		if (IsSameThread(MipDataProvider->GetCancelThread(), Context.CurrentThread))
		{
			MipDataProvider->Cancel(SyncOptions);
		}
		switch (MipDataProvider->GetCancelThread())
		{
		case FTextureMipDataProvider::ETickThread::Async:
			NextThread = TT_Async;
			break;
		case FTextureMipDataProvider::ETickThread::Render:
			NextThread = TT_Render;
			break;
		}
	}
	if (NextThread != TT_None)
	{
		PushTask(Context, TT_None, nullptr, NextThread, SRA_UPDATE_CALLBACK(Cancel));
		return;
	}
	MipDataProviders.Empty();

	// Then cancel the mip allocator.
	if (MipAllocator)
	{
		if (IsSameThread(MipAllocator->ExecuteGetCancelThread(), Context.CurrentThread))
		{
			MipAllocator->ExecuteCancel(SyncOptions);
		}
		switch (MipAllocator->ExecuteGetCancelThread())
		{
		case FTextureMipAllocator::ETickThread::Async:
			NextThread = TT_Async;
			break;
		case FTextureMipAllocator::ETickThread::Render:
			NextThread = TT_Render;
			break;
		}
		if (NextThread != TT_None)
		{
			PushTask(Context, TT_None, nullptr, NextThread, SRA_UPDATE_CALLBACK(Cancel));
			return;
		}
		MipAllocator.Reset();
		MipInfos.Empty();
	}
}

void FTextureStreamIn::Abort()
{
	MarkAsCancelled();

	if (bIsPollingMipData || bIsPostprocessingGPUMipData)
	{
		// Prevent the update from being considered done before this is finished.
		// By checking that it was not already canceled, we make sure this doesn't get called twice.
		(new FAsyncAbortPollMipsTask(this, bIsPollingMipData, bIsPostprocessingGPUMipData))->StartBackgroundTask();
	}
}

void FTextureStreamIn::FAbortPollMipsTask::DoWork()
{
	check(PendingUpdate);
	
	// Acquire the lock of this object in order to cancel any pending IO.
	// If the object is currently being ticked, wait.
	const ETaskState PreviousTaskState = PendingUpdate->DoLock();
	for (TUniquePtr<FTextureMipDataProvider>& MipDataProvider : PendingUpdate->MipDataProviders)
	{
		check(MipDataProvider);
		if(bIsPollingMipData)
		{
			MipDataProvider->AbortPollMips();
		}
		else if (bIsPostprocessingGPUMipData)
		{
			MipDataProvider->AbortPostprocessGPUMipData();
		}
	}
	PendingUpdate->DoUnlock(PreviousTaskState);
}
