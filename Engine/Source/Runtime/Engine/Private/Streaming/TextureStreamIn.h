// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
FTextureStreamIn.h: Implement a generic texture stream in strategy.
=============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "RenderAssetUpdate.h"
#include "TextureMipAllocator.h"
#include "Streaming/TextureMipDataProvider.h"

class UTexture;

/**
* FTextureStreamIn implements a generic framework to stream in textures while decoupling
* the source of the mip data (FTextureMipDataProvider) and the creation steps (FTextureMipAllocator) of the new texture.
* It is also texture type independent so it can be used as a base class for any streaming implementation.
*
* The engine uses some private implementations to handle DDC and cooked content files, as well as the different texture creation strategy,
* but allows for custom implementations of FTextureMipDataProvider (which is an ENGINE_API) through UTextureMipDataProviderFactory (of type UAssetUserData). 
* Those implementation can choose to override the default mip content for specific mips (see FTextureMipDataProvider::GetMips())
*/
class FTextureStreamIn : public TRenderAssetUpdate<FTextureUpdateContext>
{
public:

	FTextureStreamIn(const UTexture* InTexture, FTextureMipAllocator* InMipAllocator, FTextureMipDataProvider* InCustomMipDataProvider, FTextureMipDataProvider* InDefaultMipDataProvider);
	~FTextureStreamIn();

	// Route FTextureMipDataProvider::Init()
	void InitMipDataProviders(const FContext& Context);
	
	// Route FTextureMipAllocator::AllocateMips()
	void AllocateNewMips(const FContext& Context);
	
	// Route FTextureMipDataProvider::GetMips()
	void GetMipData(const FContext& Context);

	// Route FTextureMipDataProvider::PollMips()
	void PollMipData(const FContext& Context);

	// Route FTextureMipAllocator::UploadMips()
	void UploadNewMips(const FContext& Context);

	// Route FTextureMipDataProvider::PostprocessGPUMipData
	void PostprocessGPUMipData(const FContext& Context);

	// Update the texture resource with the new RHI texture. Update is completed after this step and can not be canceled afterward.
	void FinalizeNewMips(const FContext& Context);

	// Route FTextureMipDataProvider::CleanUp()
	void CleanUpMipDataProviders(const FContext& Context);

	// Execute the cancellation for both MipAllocator and MipDataProviders.
	void Cancel(const FContext& Context);

	// Start an async task to cancel pending IO requests.
	void Abort() final override;

protected:

	// The first mip to be processed for DoGetMipData(). Starts at PendingFirstLODIdx and goes up to CurrentFirstLODIdx.
	int32 StartingMipIndex = INDEX_NONE;

	TUniquePtr<FTextureMipAllocator> MipAllocator;
	FTextureMipDataProviderArray MipDataProviders;

	FTextureUpdateSyncOptions SyncOptions;
	FTextureMipInfoArray MipInfos;

	// The actual implementation of the update steps which redirect to FTextureMipAllocator and FTextureMipDataProvider.
	void DoInitMipDataProviders(const FContext& Context);
	bool DoAllocateNewMips(const FContext& Context);
	void DoGetMipData(const FContext& Context);
	bool DoPollMipData(const FContext& Context);
	bool DoUploadNewMips(const FContext& Context);
	bool DoPostprocessGPUMipData(const FContext& Context);
	bool DoFinalizeNewMips(const FContext& Context);
	void DoCleanUpMipDataProviders(const FContext& Context);

	EThreadType GetMipDataProviderThread(FTextureMipDataProvider::ETickState TickState) const; 
	EThreadType GetMipAllocatorThread(FTextureMipAllocator::ETickState TickState) const;

	EThreadType GetCancelThread() const;

private:

	// An async task that will call FTextureMipDataProvider::AbortPollMips() on each mip data provider upon cancellation.
	class FAbortPollMipsTask : public FNonAbandonableTask
	{
	public:
		FAbortPollMipsTask(FTextureStreamIn* InPendingUpdate, bool bInIsPollingMipData, bool bInIsPostprocessingGPUMipData) 
			: PendingUpdate(InPendingUpdate)
			, bIsPollingMipData(bInIsPollingMipData)
			, bIsPostprocessingGPUMipData(bInIsPostprocessingGPUMipData)
		{
		}
		void DoWork();
		FORCEINLINE TStatId GetStatId() const
		{
			RETURN_QUICK_DECLARE_CYCLE_STAT(FAbortPollMipsTask, STATGROUP_ThreadPoolAsyncTasks);
		}
	protected:
		TRefCountPtr<FTextureStreamIn> PendingUpdate;
		bool bIsPollingMipData;
		bool bIsPostprocessingGPUMipData;
	};

	bool bIsPollingMipData = false; // Used to prevent indirecting any pointer in the abort task.
	bool bIsPostprocessingGPUMipData = false; // Used to prevent indirecting any pointer in the abort task.
	typedef FAutoDeleteAsyncTask<FAbortPollMipsTask> FAsyncAbortPollMipsTask;
	friend class FAbortPollMipsTask;

	// Whether FTextureMipAllocator::ETickThread is the same thread as EThreadType (as int32).
	FORCEINLINE static bool IsSameThread(FTextureMipAllocator::ETickThread TickThread, int32 TaskThread);

	// Whether FTextureMipDataProvider::ETickThread is the same thread as EThreadType (as int32).
	FORCEINLINE static bool IsSameThread(FTextureMipDataProvider::ETickThread TickThread, int32 TaskThread);
};

