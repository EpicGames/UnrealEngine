// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
TextureStreamOut.cpp : Implement a generic texture stream out strategy.
=============================================================================*/

#include "Streaming/TextureStreamOut.h"
#include "Engine/Texture.h"
#include "Streaming/RenderAssetUpdate.inl"
#include "Streaming/TextureMipAllocator.h"

bool FTextureStreamOut::IsSameThread(FTextureMipAllocator::ETickThread TickThread, int32 TaskThread)
{
	if (TaskThread == TT_Async)
	{
		return TickThread == FTextureMipAllocator::ETickThread::Async;
	}
	else // Expected to be called from either the async thread or the renderthread.
	{
		check(TaskThread == TT_Render);
		return TickThread == FTextureMipAllocator::ETickThread::Render;
	}
}

FTextureStreamOut::FTextureStreamOut(const UTexture* InTexture, FTextureMipAllocator* InMipAllocator)
	: TRenderAssetUpdate<FTextureUpdateContext>(InTexture)
{
	check(InMipAllocator);

	// Init the allocator.
	MipAllocator.Reset(InMipAllocator);

	// Init sync options
	SyncOptions.bSnooze = &bDeferExecution;
	SyncOptions.Counter = &TaskSynchronization;
	SyncOptions.RescheduleCallback = [this]() 
	{ 
		if (!IsLocked())
		{
			Tick(TT_None); 
		}
	};

	// Schedule the first update step.
	FContext Context(InTexture, TT_None);
	const EThreadType NextThread = GetMipAllocatorThread(FTextureMipAllocator::ETickState::AllocateMips);
	if (NextThread != TT_None)
	{
		PushTask(Context, NextThread, SRA_UPDATE_CALLBACK(AllocateNewMips), GetCancelThread(), SRA_UPDATE_CALLBACK(Cancel));
	}
	else // Otherwise if it is impossible to allocate the new mips, abort.
	{
		MarkAsCancelled();
		PushTask(Context, TT_None, nullptr, GetCancelThread(), SRA_UPDATE_CALLBACK(Cancel));
	}
}

FTextureStreamOut::~FTextureStreamOut()
{
}

FRenderAssetUpdate::EThreadType FTextureStreamOut::GetMipAllocatorThread(FTextureMipAllocator::ETickState TickState) const
{
	check(MipAllocator);
	if (!IsCancelled() && MipAllocator->GetNextTickState() == TickState)
	{
		switch (MipAllocator->GetNextTickThread())
		{
		case FTextureMipAllocator::ETickThread::Async:
			return TT_Async;
		case FTextureMipAllocator::ETickThread::Render:
			return TT_Render;
		default:
			check(false);
		}
	}
	return TT_None;
}

FRenderAssetUpdate::EThreadType FTextureStreamOut::GetCancelThread() const
{
	// Cancel the mip allocator.
	if (MipAllocator)
	{
		switch (MipAllocator->ExecuteGetCancelThread())
		{
		case FTextureMipAllocator::ETickThread::Async:
			return TT_Async;
		case FTextureMipAllocator::ETickThread::Render:
			return TT_Render;
		}
	}

	// If both are cancelled, then run the final cleanup on the async thread.
	return TT_Async;
}

// ****************************
// **** Update Steps Work *****
// ****************************

bool FTextureStreamOut::DoAllocateNewMips(const FContext& Context)
{
	check(MipAllocator && IsSameThread(MipAllocator->GetNextTickThread(), Context.CurrentThread));
	FTextureMipInfoArray MipInfos;
	return MipAllocator->AllocateMips(Context, MipInfos, SyncOptions);
}

bool FTextureStreamOut::DoUploadNewMips(const FContext& Context)
{
	check(MipAllocator && IsSameThread(MipAllocator->GetNextTickThread(), Context.CurrentThread));
	return MipAllocator->UploadMips(Context, SyncOptions);
}

bool FTextureStreamOut::DoFinalizeNewMips(const FContext& Context)
{
	check(MipAllocator && IsSameThread(FTextureMipAllocator::ETickThread::Render, Context.CurrentThread));
	return MipAllocator->FinalizeMips(Context, SyncOptions);
}

// ****************************
// ******* Update Steps *******
// ****************************

void FTextureStreamOut::AllocateNewMips(const FContext& Context)
{
	DECLARE_SCOPE_CYCLE_COUNTER(TEXT("FTextureStreamOut::AllocateNewMips"), STAT_TextureStreamOut_AllocateNewMips, STATGROUP_StreamingDetails);

	// Execute
	if (!DoAllocateNewMips(Context))
	{
		MarkAsCancelled();
	}

	// Schedule the next update step.
	EThreadType NextThread = GetMipAllocatorThread(FTextureMipAllocator::ETickState::AllocateMips);
	if (NextThread != TT_None) // Loop on this state.
	{
		PushTask(Context, NextThread, SRA_UPDATE_CALLBACK(AllocateNewMips), GetCancelThread(), SRA_UPDATE_CALLBACK(Cancel));
	}
	else if (!IsCancelled())
	{
		NextThread = GetMipAllocatorThread(FTextureMipAllocator::ETickState::UploadMips);
		// All mips must be handled before moving to next stage.
		if (NextThread != TT_None)
		{
			PushTask(Context, NextThread, SRA_UPDATE_CALLBACK(UploadNewMips), GetCancelThread(), SRA_UPDATE_CALLBACK(Cancel));
		}
		else // Otherwise nothing to execute but schedule next phase (edge case).
		{
			PushTask(Context, EThreadType::TT_Render, SRA_UPDATE_CALLBACK(FinalizeNewMips), GetCancelThread(), SRA_UPDATE_CALLBACK(Cancel));
		}
	}
	else
	{
		PushTask(Context, TT_None, nullptr, GetCancelThread(), SRA_UPDATE_CALLBACK(Cancel));
	}
}

void FTextureStreamOut::UploadNewMips(const FContext& Context)
{
	DECLARE_SCOPE_CYCLE_COUNTER(TEXT("FTextureStreamOut::UploadNewMips"), STAT_TextureStreamOut_UploadNewMips, STATGROUP_StreamingDetails);

	// Execute
	if (!DoUploadNewMips(Context))
	{
		MarkAsCancelled();
	}

	// Schedule the next update step.
	EThreadType NextThread = GetMipAllocatorThread(FTextureMipAllocator::ETickState::UploadMips);
	if (NextThread != TT_None) // Loop on this state.
	{
		PushTask(Context, NextThread, SRA_UPDATE_CALLBACK(UploadNewMips), GetCancelThread(), SRA_UPDATE_CALLBACK(Cancel));
	}
	else
	{
		if (!IsCancelled())
		{
			PushTask(Context, EThreadType::TT_Render, SRA_UPDATE_CALLBACK(FinalizeNewMips), GetCancelThread(), SRA_UPDATE_CALLBACK(Cancel));
		}
		else
		{
			PushTask(Context, TT_None, nullptr, GetCancelThread(), SRA_UPDATE_CALLBACK(Cancel));
		}
	}
}

void FTextureStreamOut::FinalizeNewMips(const FContext& Context)
{
	DECLARE_SCOPE_CYCLE_COUNTER(TEXT("FTextureStreamOut::FinalizeNewMips"), STAT_TextureStreamOut_FinalizeNewMips, STATGROUP_StreamingDetails);

	// Execute
	if (DoFinalizeNewMips(Context))
	{
		MarkAsSuccessfullyFinished();
	}
	else
	{
		MarkAsCancelled();
	}

	/// Release the mip allocator.
	MipAllocator.Reset();
}

void FTextureStreamOut::Cancel(const FContext& Context)
{
	DECLARE_SCOPE_CYCLE_COUNTER(TEXT("FTextureStreamOut::Cancel"), STAT_TextureStreamOut_Cancel, STATGROUP_StreamingDetails);

	EThreadType NextThread = TT_None;

	// Then cancel the mip allocator.
	if (MipAllocator)
	{
		if (IsSameThread(MipAllocator->ExecuteGetCancelThread(), Context.CurrentThread))
		{
			MipAllocator->ExecuteCancel(SyncOptions);
		}
		switch (MipAllocator->ExecuteGetCancelThread())
		{
		case FTextureMipAllocator::ETickThread::Async:
			NextThread = TT_Async;
			break;
		case FTextureMipAllocator::ETickThread::Render:
			NextThread = TT_Render;
			break;
		}
		if (NextThread != TT_None)
		{
			PushTask(Context, TT_None, nullptr, NextThread, SRA_UPDATE_CALLBACK(Cancel));
			return;
		}
		MipAllocator.Reset();
	}
}
