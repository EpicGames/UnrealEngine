// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
TextureStreamOut.h: Implement a generic texture stream out strategy.
=============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "RenderAssetUpdate.h"
#include "TextureMipAllocator.h"

class UTexture;

/**
* FTextureStreamOut implements a generic framework to stream out using  FTextureMipAllocator.
*/
class FTextureStreamOut : public TRenderAssetUpdate<FTextureUpdateContext>
{
public:

	FTextureStreamOut(const UTexture* InTexture, FTextureMipAllocator* InMipAllocator);
	~FTextureStreamOut();
	
	// Route FTextureMipAllocator::AllocateMips()
	void AllocateNewMips(const FContext& Context);

	// Route FTextureMipAllocator::UploadMips()
	void UploadNewMips(const FContext& Context);
	
	// Route FTextureMipAllocator::FinalizeMips()
	void FinalizeNewMips(const FContext& Context);

	// Execute the cancellation for both MipAllocator.
	void Cancel(const FContext& Context);

protected:

	TUniquePtr<FTextureMipAllocator> MipAllocator;

	FTextureUpdateSyncOptions SyncOptions;

	// The actual implementation of the update steps which redirect to FTextureMipAllocator and FTextureMipDataProvider.
	bool DoAllocateNewMips(const FContext& Context);
	bool DoUploadNewMips(const FContext& Context);
	bool DoFinalizeNewMips(const FContext& Context);

	EThreadType GetMipAllocatorThread(FTextureMipAllocator::ETickState TickState) const;

	EThreadType GetCancelThread() const;

private:

	// Whether FTextureMipAllocator::ETickThread is the same thread as EThreadType (as int32).
	FORCEINLINE static bool IsSameThread(FTextureMipAllocator::ETickThread TickThread, int32 TaskThread);
};

