// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
TextureStreamingBuild.cpp : Contains definitions to build texture streaming data.
=============================================================================*/

#include "DebugViewModeHelpers.h"
#include "MeshUVChannelInfo.h"
#include "Misc/ScopedSlowTask.h"
#include "Engine/Level.h"
#include "Engine/World.h"
#include "Components/PrimitiveComponent.h"
#include "Materials/Material.h"
#include "Materials/MaterialInterface.h"
#include "Misc/FeedbackContext.h"
#include "Streaming/TextureStreamingHelpers.h"
#include "Streaming/ActorTextureStreamingBuildDataComponent.h"
#include "UObject/Package.h"
#include "UObject/UObjectIterator.h"
#include "Logging/MessageLog.h"
#include "Misc/UObjectToken.h"
#include "Engine/SkeletalMesh.h"
#include "Engine/StaticMesh.h"
#if WITH_EDITOR
#include "Engine/Texture2D.h"
#include "Engine/Texture2DArray.h"
#include "Engine/VolumeTexture.h"
#include "Interfaces/ITargetPlatform.h"
#include "Interfaces/ITargetPlatformManagerModule.h"
#else
#include "Engine/Texture.h"
#include "UnrealEngine.h"
#endif


DEFINE_LOG_CATEGORY(TextureStreamingBuild);
#define LOCTEXT_NAMESPACE "TextureStreamingBuild"

#if WITH_EDITOR

static const uint32 GInvalidPackedTextureStreamingQualityLevelFeatureLevel = 0xFFFFFFFF;

ENGINE_API uint32 GetPackedTextureStreamingQualityLevelFeatureLevel(EMaterialQualityLevel::Type InQualityLevel, ERHIFeatureLevel::Type InFeatureLevel)
{
	return ((uint32)InQualityLevel << 16) | ((uint32)InFeatureLevel & 0xFFFF);
}

static uint32 ComputeHashTextureStreamingDataForActor(AActor* InActor)
{
	uint32 Hash = 0;
	if (UActorTextureStreamingBuildDataComponent* BuiltDataComponent = InActor->FindComponentByClass<UActorTextureStreamingBuildDataComponent>())
	{
		Hash = FCrc::TypeCrc32(BuiltDataComponent->ComputeHash(), Hash);
	}
	
	TInlineComponentArray<UPrimitiveComponent*> Primitives;
	InActor->GetComponents(Primitives);

	TArray<uint32> PrimitiveHashes;
	for (UPrimitiveComponent* Primitive : Primitives)
	{
		if (Primitive)
		{
			PrimitiveHashes.Add(FCrc::TypeCrc32(Primitive->ComputeHashTextureStreamingBuiltData()));
		}
	}
	PrimitiveHashes.Sort();
	for (uint32 PrimitiveHash : PrimitiveHashes)
	{
		Hash = FCrc::TypeCrc32(PrimitiveHash, Hash);
	}
	return Hash;
}

void BuildActorTextureStreamingData(AActor* InActor, EMaterialQualityLevel::Type InQualityLevel, ERHIFeatureLevel::Type InFeatureLevel)
{
	if (!InActor || InActor->IsTemplate())
	{
		return;
	}
	uint32 OldHash = ComputeHashTextureStreamingDataForActor(InActor);
	
	TSet<FGuid> DummyResourceGuids;
	UActorTextureStreamingBuildDataComponent* BuiltDataComponent = InActor->FindComponentByClass<UActorTextureStreamingBuildDataComponent>();
	if (!BuiltDataComponent)
	{
		BuiltDataComponent = NewObject<UActorTextureStreamingBuildDataComponent>(InActor, NAME_None, RF_Transactional);
		InActor->AddInstanceComponent(BuiltDataComponent);
	}
	BuiltDataComponent->InitializeTextureStreamingContainer(GetPackedTextureStreamingQualityLevelFeatureLevel(InQualityLevel, InFeatureLevel));

	TInlineComponentArray<UPrimitiveComponent*> Primitives;
	InActor->GetComponents(Primitives);
	for (UPrimitiveComponent* Primitive : Primitives)
	{
		if (Primitive)
		{
			Primitive->BuildTextureStreamingData(TSB_ActorBuild, InQualityLevel, InFeatureLevel, DummyResourceGuids);
		}
	}

	// If actor's texture streaming built data has changed, dirty its package
	uint32 NewHash = ComputeHashTextureStreamingDataForActor(InActor);
	if (NewHash != OldHash)
	{
		InActor->GetPackage()->MarkPackageDirty();
	}
}

bool BuildLevelTextureStreamingComponentDataFromActors(ULevel* InLevel)
{
	// If Level already contains texture streaming built data, keep it as it was built in the editor.
	if (!InLevel->StreamingTextureGuids.IsEmpty())
	{
		return true;
	}

	TRACE_CPUPROFILER_EVENT_SCOPE(BuildLevelTextureStreamingComponentDataFromActors);

	// Find actors with texture streaming built data (built with BuildActorTextureStreamingData) and remap this data to the level
	bool bIsLevelInitialized = false;
	for (AActor* Actor : InLevel->Actors)
	{
		UActorTextureStreamingBuildDataComponent* BuiltDataComponent = Actor ? Actor->FindComponentByClass<UActorTextureStreamingBuildDataComponent>() : nullptr;
		if (!BuiltDataComponent)
		{
			continue;
		}

		uint32 ComponentPackedInfo = BuiltDataComponent->GetPackedTextureStreamingQualityLevelFeatureLevel();
		if (!bIsLevelInitialized)
		{
			InLevel->InitializeTextureStreamingContainer(ComponentPackedInfo);
			bIsLevelInitialized = true;
		}
		else if (InLevel->PackedTextureStreamingQualityLevelFeatureLevel != ComponentPackedInfo)
		{
			// We don't support mixed versions of quality/feature levels
			InLevel->PackedTextureStreamingQualityLevelFeatureLevel = GInvalidPackedTextureStreamingQualityLevelFeatureLevel;
			return false;
		}

		TInlineComponentArray<UPrimitiveComponent*> Primitives;
		Actor->GetComponents(Primitives);
		for (UPrimitiveComponent* Primitive : Primitives)
		{
			if (Primitive && Primitive->bIsActorTextureStreamingBuiltData)
			{
				// Remap primitive's texture streaming built data from actor based to level based
				Primitive->bIsValidTextureStreamingBuiltData = Primitive->RemapActorTextureStreamingBuiltDataToLevel(BuiltDataComponent);
			}
		}
	}
	return true;
}

bool GetTextureIsStreamable(const UTexture& Texture)
{
	const ITargetPlatform& CurrentPlatform = *GetTargetPlatformManagerRef().GetRunningTargetPlatform();

	bool bStreamable = GetTextureIsStreamableOnPlatform(Texture, CurrentPlatform);
	return bStreamable;
}

bool GetTextureIsStreamableOnPlatform(const UTexture& Texture, const ITargetPlatform& TargetPlatform)
{
	const bool bPlatformSupportsTextureStreaming = TargetPlatform.SupportsFeature(ETargetPlatformFeatures::TextureStreaming);
	if ( ! bPlatformSupportsTextureStreaming )
	{
		// IsCandidateForTextureStreamingOnPlatformDuringCook also checks this
		return false;
	}

	bool bStreamable = false;
	if (Texture.IsA(UTexture2DArray::StaticClass()))
	{
		bStreamable = GSupportsTexture2DArrayStreaming;
	}
	else if (Texture.IsA(UVolumeTexture::StaticClass()))
	{
		bStreamable = GSupportsVolumeTextureStreaming;
	}
	else if (Texture.IsA(UTexture2D::StaticClass()))
	{
		bStreamable = true;
	}

	bStreamable &= Texture.IsCandidateForTextureStreamingOnPlatformDuringCook(&TargetPlatform);
	return bStreamable;
}

#else

static bool GetUnpackedTextureStreamingQualityLevelFeatureLevel(uint32 InPackedValue, EMaterialQualityLevel::Type& OutQualityLevel, ERHIFeatureLevel::Type& OutFeatureLevel)
{
	uint32 QualityLevel = InPackedValue >> 16;
	uint32 FeatureLevel = InPackedValue & 0xFFFF;
	bool bIsValidQualityLevel = QualityLevel <= (uint32)EMaterialQualityLevel::Num;
	bool bIsValidFeatureLevel = FeatureLevel <= (uint32)ERHIFeatureLevel::Num;
	if (bIsValidQualityLevel && bIsValidFeatureLevel)
	{
		OutQualityLevel = (EMaterialQualityLevel::Type)QualityLevel;
		OutFeatureLevel = (ERHIFeatureLevel::Type)FeatureLevel;
		if (OutQualityLevel == EMaterialQualityLevel::Num)
		{
			OutQualityLevel = GetCachedScalabilityCVars().MaterialQualityLevel;
		}
		if (OutFeatureLevel == ERHIFeatureLevel::Num)
		{
			OutFeatureLevel = GMaxRHIFeatureLevel;
		}
		return true;
	}
	return false;
}

#endif

ENGINE_API bool BuildTextureStreamingComponentData(UWorld* InWorld, EMaterialQualityLevel::Type QualityLevel, ERHIFeatureLevel::Type FeatureLevel, bool bFullRebuild, FSlowTask& BuildTextureStreamingTask)
{
#if WITH_EDITORONLY_DATA
	if (!InWorld)
	{
		return false;
	}

	const int32 NumActorsInWorld = GetNumActorsInWorld(InWorld);
	if (!NumActorsInWorld)
	{
		BuildTextureStreamingTask.EnterProgressFrame();
		// Can't early exit here as Level might need reset.
		// return true;
	}

	const double StartTime = FPlatformTime::Seconds();
	const float OneOverNumActorsInWorld = 1.f / (float)FMath::Max<int32>(NumActorsInWorld, 1); // Prevent div by 0

	// Used to reset per level index for textures.
	TArray<UTexture*> AllTextures;
	for (FThreadSafeObjectIterator Iter(UTexture::StaticClass()); Iter && bFullRebuild; ++Iter)
	{
		UTexture* Texture = Cast<UTexture>(*Iter);
		if (Texture)
		{
			AllTextures.Add(Texture);
		}
	}

	FScopedSlowTask SlowTask(1.f, (LOCTEXT("TextureStreamingBuild_ComponentDataUpdate", "Updating Component Data")));
	
	for (int32 LevelIndex = 0; LevelIndex < InWorld->GetNumLevels(); LevelIndex++)
	{
		ULevel* Level = InWorld->GetLevel(LevelIndex);
		if (!Level)
		{
			continue;
		}

		const bool bHadBuildData = Level->StreamingTextureGuids.Num() > 0 || Level->TextureStreamingResourceGuids.Num() > 0;

		Level->NumTextureStreamingUnbuiltComponents = 0;

		// When not rebuilding everything, we can't update those as we don't know how the current build data was computed.
		// Consequently, partial rebuilds are not allowed to recompute everything. What something is missing and can not be built,
		// the BuildStreamingData will return false in which case we increment NumTextureStreamingUnbuiltComponents.
		// This allows to keep track of full rebuild requirements.
		if (bFullRebuild)
		{
			Level->InitializeTextureStreamingContainer(GetPackedTextureStreamingQualityLevelFeatureLevel(QualityLevel, FeatureLevel));
		}

		TSet<FGuid> ResourceGuids;
		TSet<FGuid> DummyResourceGuids;

		for (AActor* Actor : Level->Actors)
		{
			BuildTextureStreamingTask.EnterProgressFrame(OneOverNumActorsInWorld);
			SlowTask.EnterProgressFrame(OneOverNumActorsInWorld);
			if (GWarn->ReceivedUserCancel())
			{
				return false;
			}

			// Check the actor after incrementing the progress.
			if (!Actor)
			{
				continue; 
			}

			TInlineComponentArray<UPrimitiveComponent*> Primitives;
			Actor->GetComponents(Primitives);

			for (UPrimitiveComponent* Primitive : Primitives)
			{
				if (!Primitive)
				{
					continue;
				}
				else if (!Primitive->HasAnyFlags(RF_Transactional))
				{
					// For non transactional primitives, like the one created from blueprints, we tolerate fails and we also don't store the guids.
					Primitive->BuildTextureStreamingData(bFullRebuild ? TSB_MapBuild : TSB_ViewMode, QualityLevel, FeatureLevel, DummyResourceGuids);
				}
				else if (!Primitive->BuildTextureStreamingData(bFullRebuild ? TSB_MapBuild : TSB_ViewMode, QualityLevel, FeatureLevel, ResourceGuids))
				{
					++Level->NumTextureStreamingUnbuiltComponents;
				}
			}
		}

		if (bFullRebuild)
		{
			// Reset LevelIndex to default for next use and build the level Guid array.
			for (UTexture* Texture : AllTextures)
			{
				checkSlow(Texture);
				Texture->LevelIndex = INDEX_NONE;
			}

			// Cleanup the asset references.
			ResourceGuids.Remove(FGuid()); // Remove the invalid guid.
			for (const FGuid& ResourceGuid : ResourceGuids)
			{
				Level->TextureStreamingResourceGuids.Add(ResourceGuid);
			}

			// Mark for resave if and only if rebuilding everything and also if data was updated.
			const bool bHasBuildData = Level->StreamingTextureGuids.Num() > 0 || Level->TextureStreamingResourceGuids.Num() > 0;
			if (bHadBuildData || bHasBuildData)
			{
				Level->MarkPackageDirty();
			}
		}
	}

	// Update TextureStreamer
	ULevel::BuildStreamingData(InWorld);

	UE_LOGF(TextureStreamingBuild, Display, "Build Texture Streaming took %.3f seconds.", FPlatformTime::Seconds() - StartTime);
	return true;
#else
	UE_LOGF(TextureStreamingBuild, Fatal,"Build Texture Streaming should not be called on a console");
	return false;
#endif
}

#undef LOCTEXT_NAMESPACE

uint32 PackRelativeBox(const FVector& RefOrigin, const FVector& RefExtent, const FVector& Origin, const FVector& Extent)
{
	const FVector RefMin = RefOrigin - RefExtent;
	// 15.5 and 31.5 have the / 2 scale included 
	const FVector PackScale = FVector(15.5f, 15.5f, 31.5f) / RefExtent.ComponentMax(FVector(UE_KINDA_SMALL_NUMBER, UE_KINDA_SMALL_NUMBER, UE_KINDA_SMALL_NUMBER));

	const FVector Min = Origin - Extent;
	const FVector Max = Origin + Extent;

	const FVector RelMin = (Min - RefMin) * PackScale;
	const FVector RelMax = (Max - RefMin) * PackScale;

	const uint32 PackedMinX = (uint32)FMath::Clamp<int32>(FMath::FloorToInt(RelMin.X), 0, 31);
	const uint32 PackedMinY = (uint32)FMath::Clamp<int32>(FMath::FloorToInt(RelMin.Y), 0, 31);
	const uint32 PackedMinZ = (uint32)FMath::Clamp<int32>(FMath::FloorToInt(RelMin.Z), 0, 63);

	const uint32 PackedMaxX = (uint32)FMath::Clamp<int32>(FMath::CeilToInt(RelMax.X), 0, 31);
	const uint32 PackedMaxY = (uint32)FMath::Clamp<int32>(FMath::CeilToInt(RelMax.Y), 0, 31);
	const uint32 PackedMaxZ = (uint32)FMath::Clamp<int32>(FMath::CeilToInt(RelMax.Z), 0, 63);

	return PackedMinX | (PackedMinY << 5) | (PackedMinZ << 10) | (PackedMaxX << 16) | (PackedMaxY << 21) | (PackedMaxZ << 26);
}

uint32 PackRelativeBox(const FBox& RefBox, const FBox& Box)
{
	// 15.5 and 31.5 have the / 2 scale included 
	const FVector PackScale = FVector(15.5f, 15.5f, 31.5f) / RefBox.GetExtent().ComponentMax(FVector(UE_KINDA_SMALL_NUMBER, UE_KINDA_SMALL_NUMBER, UE_KINDA_SMALL_NUMBER));

	const FVector RelMin = (Box.Min - RefBox.Min) * PackScale;
	const FVector RelMax = (Box.Max - RefBox.Min) * PackScale;

	const uint32 PackedMinX = (uint32)FMath::Clamp<int32>(FMath::FloorToInt(RelMin.X), 0, 31);
	const uint32 PackedMinY = (uint32)FMath::Clamp<int32>(FMath::FloorToInt(RelMin.Y), 0, 31);
	const uint32 PackedMinZ = (uint32)FMath::Clamp<int32>(FMath::FloorToInt(RelMin.Z), 0, 63);

	const uint32 PackedMaxX = (uint32)FMath::Clamp<int32>(FMath::CeilToInt(RelMax.X), 0, 31);
	const uint32 PackedMaxY = (uint32)FMath::Clamp<int32>(FMath::CeilToInt(RelMax.Y), 0, 31);
	const uint32 PackedMaxZ = (uint32)FMath::Clamp<int32>(FMath::CeilToInt(RelMax.Z), 0, 63);

	return PackedMinX | (PackedMinY << 5) | (PackedMinZ << 10) | (PackedMaxX << 16) | (PackedMaxY << 21) | (PackedMaxZ << 26);
}

void UnpackRelativeBox(const FBoxSphereBounds& InRefBounds, uint32 InPackedRelBox, FBoxSphereBounds& OutBounds)
{
	if (InPackedRelBox == PackedRelativeBox_Identity)
	{
		OutBounds = InRefBounds;
	}
	else if (InRefBounds.SphereRadius > 0)
	{
		const uint32 PackedMinX = InPackedRelBox & 31;
		const uint32 PackedMinY = (InPackedRelBox >> 5) & 31;
		const uint32 PackedMinZ = (InPackedRelBox >> 10) & 63;

		const uint32 PackedMaxX = (InPackedRelBox >> 16) & 31;
		const uint32 PackedMaxY = (InPackedRelBox >> 21) & 31;
		const uint32 PackedMaxZ = (InPackedRelBox >> 26) & 63;

		const FVector RefMin = InRefBounds.Origin - InRefBounds.BoxExtent;
		// 15.5 and 31.5 have the / 2 scale included 
		const FVector UnpackScale = InRefBounds.BoxExtent.ComponentMax(FVector(UE_KINDA_SMALL_NUMBER, UE_KINDA_SMALL_NUMBER, UE_KINDA_SMALL_NUMBER)) / FVector(15.5f, 15.5f, 31.5f);

		const FVector Min = FVector((float)PackedMinX, (float)PackedMinY, (float)PackedMinZ) * UnpackScale + RefMin;
		const FVector Max = FVector((float)PackedMaxX, (float)PackedMaxY, (float)PackedMaxZ) * UnpackScale + RefMin;

		OutBounds.Origin = .5 * (Min + Max);
		OutBounds.BoxExtent = .5 * (Max - Min);
		OutBounds.SphereRadius = OutBounds.BoxExtent.Size();
	}
	else // In that case the ref bounds is 0, so any relative bound is also 0.
	{
		OutBounds.Origin = FVector::ZeroVector;
		OutBounds.BoxExtent = FVector::ZeroVector;
		OutBounds.SphereRadius = 0;
	}
}

#if WITH_EDITOR
void FStreamingTextureBuildInfo::PackFrom(ITextureStreamingContainer* TextureStreamingContainer, const FBoxSphereBounds& RefBounds, const FStreamingRenderAssetPrimitiveInfo& Info)
{
	check(TextureStreamingContainer);

	PackedRelativeBox = PackRelativeBox(RefBounds.Origin, RefBounds.BoxExtent, Info.Bounds.Origin, Info.Bounds.BoxExtent);

	UTexture* Texture = Cast<UTexture>(Info.RenderAsset);
	check(Texture);
	TextureLevelIndex = TextureStreamingContainer->RegisterStreamableTexture(Texture);
	TexelFactor = Info.TexelFactor;
}

uint32 FStreamingTextureBuildInfo::ComputeHash() const
{
	return FCrc::TypeCrc32(PackedRelativeBox, FCrc::TypeCrc32(TextureLevelIndex, FCrc::TypeCrc32(TexelFactor, 0)));
}
#endif

static int32 GUseTextureStreamingBuiltData = 1;
static FAutoConsoleVariableRef CVarUseTextureStreamingBuiltData(
	TEXT("r.Streaming.UseTextureStreamingBuiltData"),
	GUseTextureStreamingBuiltData,
	TEXT("Turn on/off usage of texture streaming built data (0 to turn off)."));

FStreamingTextureLevelContext::FStreamingTextureLevelContext(EMaterialQualityLevel::Type InQualityLevel, const UPrimitiveComponent* Primitive)
	: TextureGuidToLevelIndex(nullptr)
	, bUseRelativeBoxes(false)
	, BuildDataTimestamp(0)
	, ComponentBuildData(nullptr)
	, bForceNoUseBuiltData(false)
{
	ULevel* Level = Primitive && Primitive->GetOwner() ? Primitive->GetOwner()->GetLevel() : nullptr;
	UpdateQualityAndFeatureLevel(InQualityLevel, GMaxRHIFeatureLevel, Level);
}

FStreamingTextureLevelContext::FStreamingTextureLevelContext(EMaterialQualityLevel::Type InQualityLevel, const IPrimitiveComponent* Primitive)
	: TextureGuidToLevelIndex(nullptr)
	, bUseRelativeBoxes(false)
	, BuildDataTimestamp(0)
	, ComponentBuildData(nullptr)
	, bForceNoUseBuiltData(false)
{
	ULevel* Level = nullptr;
	if (UObject* Owner = Primitive ? Primitive->GetOwner() : nullptr)
	{
		if (AActor* OwnerActor = Cast<AActor>(Owner))
		{
			Level = OwnerActor->GetLevel();
		}
	}
	UpdateQualityAndFeatureLevel(InQualityLevel, GMaxRHIFeatureLevel, Level);
}

FStreamingTextureLevelContext::FStreamingTextureLevelContext(EMaterialQualityLevel::Type InQualityLevel, ERHIFeatureLevel::Type InFeatureLevel, bool InUseRelativeBoxes)
	: TextureGuidToLevelIndex(nullptr)
	, bUseRelativeBoxes(InUseRelativeBoxes)
	, BuildDataTimestamp(0)
	, ComponentBuildData(nullptr)
	, bForceNoUseBuiltData(false)
{
	UpdateQualityAndFeatureLevel(InQualityLevel, InFeatureLevel);
}

FStreamingTextureLevelContext::FStreamingTextureLevelContext(EMaterialQualityLevel::Type InQualityLevel, const ULevel* InLevel, const TMap<FGuid, int32>* InTextureGuidToLevelIndex) 
: TextureGuidToLevelIndex(nullptr)
, bUseRelativeBoxes(false)
, BuildDataTimestamp(0)
, ComponentBuildData(nullptr)
, bForceNoUseBuiltData(false)
{
	UpdateContext(InQualityLevel, InLevel, InTextureGuidToLevelIndex);
}

void FStreamingTextureLevelContext::UpdateContext(EMaterialQualityLevel::Type InQualityLevel, const ULevel* InLevel, const TMap<FGuid, int32>* InTextureGuidToLevelIndex)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FStreamingTextureLevelContext::UpdateContext);
	TextureGuidToLevelIndex = nullptr;
	bUseRelativeBoxes = false;
	BuildDataTimestamp = 0;
	ComponentBuildData = nullptr;
	LevelStreamingTextures.Reset();
	for (const FTextureBoundState& BoundState : BoundStates)
	{
		if (BoundState.Texture)
		{
			BoundState.Texture->LevelIndex = INDEX_NONE;
		}
	}
	BoundStates.Reset();

	UpdateQualityAndFeatureLevel(InQualityLevel, GMaxRHIFeatureLevel, InLevel);

	if (InLevel)
	{
		if (CanUseTextureStreamingBuiltData() && InTextureGuidToLevelIndex && InLevel->StreamingTextureGuids.Num() > 0 && InLevel->StreamingTextureGuids.Num() == InTextureGuidToLevelIndex->Num())
		{
			bUseRelativeBoxes = !InLevel->bTextureStreamingRotationChanged;
			TextureGuidToLevelIndex = InTextureGuidToLevelIndex;

			// Extra transient data for each texture.
			BoundStates.AddZeroed(InLevel->StreamingTextureGuids.Num());

			if (InLevel->StreamingTextures.Num() == InLevel->StreamingTextureGuids.Num())
			{
				LevelStreamingTextures.SetNumZeroed(InLevel->StreamingTextures.Num());
				for (int TextureLevelIndex = 0; TextureLevelIndex < InLevel->StreamingTextures.Num(); ++TextureLevelIndex)
				{
					if (UTexture* Texture = FindObject<UTexture>(nullptr, *InLevel->StreamingTextures[TextureLevelIndex].ToString()))
					{
						LevelStreamingTextures[TextureLevelIndex] = Texture;
						GetBuildDataIndexRef(Texture, /*ForceUpdate*/ true);
					}
				}
			}
		}
	}
}

void FStreamingTextureLevelContext::UpdateQualityAndFeatureLevel(EMaterialQualityLevel::Type InQualityLevel, ERHIFeatureLevel::Type InFeatureLevel, const ULevel* InLevel)
{
	QualityLevel = InQualityLevel;
	FeatureLevel = InFeatureLevel;
	bIsBuiltDataValid = false;

	if (InLevel)
	{
		if (const UWorld* World = InLevel->GetWorld())
		{
			FeatureLevel = World->GetFeatureLevel();
		}
#if !WITH_EDITOR
		// Detect if quality level and feature level used to build texture streaming data matches the one of the context
		EMaterialQualityLevel::Type BuiltDataQualityLevel;
		ERHIFeatureLevel::Type BuiltDataFeatureLevel;
		if (GetUnpackedTextureStreamingQualityLevelFeatureLevel(InLevel->PackedTextureStreamingQualityLevelFeatureLevel, BuiltDataQualityLevel, BuiltDataFeatureLevel))
		{
			EMaterialQualityLevel::Type ContextQualityLevel = (QualityLevel == EMaterialQualityLevel::Num) ? GetCachedScalabilityCVars().MaterialQualityLevel : QualityLevel;
			ERHIFeatureLevel::Type ContextFeatureLevel = (FeatureLevel == ERHIFeatureLevel::Num) ? GMaxRHIFeatureLevel : FeatureLevel;

			bIsBuiltDataValid = (ContextQualityLevel == BuiltDataQualityLevel) && (ContextFeatureLevel == BuiltDataFeatureLevel);
		}
#endif
	}
}

TArray<FStreamingRenderAssetPrimitiveInfo> FStreamingTextureLevelContext::GetStreamingRenderAssetInfoWithNULLRemoval(const IPrimitiveComponent* Component)
{
	TArray<FStreamingRenderAssetPrimitiveInfo> StreamingRenderAssets;

	const bool bRegistered = Component->IsRegistered();

	// Ignore components that are fully initialized but have no scene proxy (hidden primitive or non game primitive)
	if (!bRegistered || !Component->IsRenderStateCreated() || Component->GetSceneProxy())
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FStreamingTextureLevelContext::GetStreamingRenderAssetInfoWithNULLRemoval);

		const bool bForceMipStreaming = Component->IsMipStreamingForced();

		Component->GetStreamingRenderAssetInfo(*this, StreamingRenderAssets);
		for (int32 Index = 0; Index < StreamingRenderAssets.Num(); Index++)
		{
			const FStreamingRenderAssetPrimitiveInfo& Info = StreamingRenderAssets[Index];
			if (!Info.RenderAsset || !Info.RenderAsset->IsStreamable())
			{
				StreamingRenderAssets.RemoveAtSwap(Index--, EAllowShrinking::No);
			}
			else
			{
#if DO_CHECK
				ensure(Info.TexelFactor >= 0.f
					|| Info.RenderAsset->IsA<UStaticMesh>()
					|| Info.RenderAsset->IsA<USkeletalMesh>()
					|| (Info.RenderAsset->IsA<UTexture>() && Info.RenderAsset->GetLODGroupForStreaming() == TEXTUREGROUP_Terrain_Heightmap));
#endif

				// Other wise check that everything is setup right. If the component is not yet registered, then the bound data is irrelevant.
				const bool bCanBeStreamedByDistance = Info.CanBeStreamedByDistance(bRegistered);

				if (!bForceMipStreaming && !bCanBeStreamedByDistance && Info.TexelFactor >= 0.f)
				{
					StreamingRenderAssets.RemoveAtSwap(Index--, EAllowShrinking::No);
				}
			}
		}
	}

	return StreamingRenderAssets;
}


bool FStreamingTextureLevelContext::CanUseTextureStreamingBuiltData() const
{
	return !bForceNoUseBuiltData && bIsBuiltDataValid && GUseTextureStreamingBuiltData != 0;
}

FStreamingTextureLevelContext::~FStreamingTextureLevelContext()
{
	// Reset the level indices for the next use.
	for (const FTextureBoundState& BoundState : BoundStates)
	{
		if (BoundState.Texture)
		{
			BoundState.Texture->LevelIndex = INDEX_NONE;
		}
	}
}

void FStreamingTextureLevelContext::BindBuildData(const TArray<FStreamingTextureBuildInfo>* BuildData)
{
	// Increment the component timestamp, used to know when a texture is processed by a component for the first time.
	// Using a timestamp allows to not reset state in between components.
	++BuildDataTimestamp;

	if (TextureGuidToLevelIndex && CVarStreamingUseNewMetrics.GetValueOnGameThread() != 0) // No point in binding data if there is no possible remapping.
	{
		// Process the build data in order to be able to map a texture object to the build data entry.
		ComponentBuildData = CanUseTextureStreamingBuiltData() ? BuildData : nullptr;
		if (BuildData && BoundStates.Num() > 0)
		{
			for (int32 Index = 0; Index < BuildData->Num(); ++Index)
			{
				int32 TextureLevelIndex = (*BuildData)[Index].TextureLevelIndex;
				if (BoundStates.IsValidIndex(TextureLevelIndex))
				{
					FTextureBoundState& BoundState = BoundStates[TextureLevelIndex];
					BoundState.BuildDataIndex = Index; // The index of this texture in the component build data.
					BoundState.BuildDataTimestamp = BuildDataTimestamp; // The component timestamp will indicate that the index is valid to be used.
				}
			}
		}
	}
	else
	{
		ComponentBuildData = nullptr;
	}
}

int32* FStreamingTextureLevelContext::GetBuildDataIndexRef(UTexture* Texture, bool bForceUpdate)
{
	if (ComponentBuildData || bForceUpdate) // If there is some build data to map to.
	{
		if (Texture->LevelIndex == INDEX_NONE)
		{
			check(TextureGuidToLevelIndex); // Can't bind ComponentData without the remapping.
			const int32* LevelIndex = TextureGuidToLevelIndex->Find(Texture->GetLightingGuid());
			if (LevelIndex) // If the index is found in the map, the index is valid in BoundStates
			{
				// Here we need to support the invalid case where 2 textures have the same GUID.
				// If this happens, BoundState.Texture will already be set.
				FTextureBoundState& BoundState = BoundStates[*LevelIndex];
				if (!BoundState.Texture)
				{
					Texture->LevelIndex = *LevelIndex;
					BoundState.Texture = Texture; // Update the mapping now!
				}
				else // Don't allow 2 textures to be using the same level index otherwise UTexturD::LevelIndex won't be reset properly in the destructor.
				{
					FMessageLog("AssetCheck").Error()
						->AddToken(FUObjectToken::Create(BoundState.Texture))
						->AddToken(FUObjectToken::Create(Texture))
						->AddToken(FTextToken::Create( NSLOCTEXT("AssetCheck", "TextureError_NonUniqueLightingGuid", "Same lighting guid, modify or touch any property in the texture editor to generate a new guid and fix the issue.") ) );

					// This will fallback not using the precomputed data. Note also that the other texture might be using the wrong precomputed data.
					return nullptr;
				}
			}
			else // Otherwise add a dummy entry to prevent having to search in the map multiple times.
			{
				Texture->LevelIndex = BoundStates.Add(FTextureBoundState(Texture));
			}
		}

		FTextureBoundState& BoundState = BoundStates[Texture->LevelIndex];
		check(BoundState.Texture == Texture);

		if (BoundState.BuildDataTimestamp == BuildDataTimestamp)
		{
			return &BoundState.BuildDataIndex; // Only return the bound static if it has data relative to this component.
		}
	}
	return nullptr;
}

void FStreamingTextureLevelContext::ProcessMaterial(const FBoxSphereBounds& ComponentBounds, const FPrimitiveMaterialInfo& MaterialData, float ComponentScaling, TArray<FStreamingRenderAssetPrimitiveInfo>& OutStreamingTextures, bool bIsComponentBuildDataValid)
{
	ensure(MaterialData.IsValid());

	auto ProcessTexture = [this, &ComponentBounds, &MaterialData, ComponentScaling, &OutStreamingTextures](UTexture* Texture)
	{
#if WITH_EDITOR
		if (GetTextureIsStreamable(*Texture))
#else
		if (Texture->IsStreamable())
#endif
		{
			float TextureDensity = MaterialData.Material->GetTextureDensity(Texture->GetFName(), *MaterialData.UVChannelData);

			if (!TextureDensity)
			{
				// Fallback assuming a sampling scale of 1 using the UV channel 0;
				TextureDensity = MaterialData.UVChannelData->LocalUVDensities[0];
			}

			if (TextureDensity)
			{
				FStreamingRenderAssetPrimitiveInfo& Info = OutStreamingTextures.AddDefaulted_GetRef();
				Info.RenderAsset = Texture;
				Info.TexelFactor = TextureDensity * ComponentScaling;
				Info.PackedRelativeBox = bUseRelativeBoxes ? MaterialData.PackedRelativeBox : PackedRelativeBox_Identity;
				UnpackRelativeBox(ComponentBounds, Info.PackedRelativeBox, Info.Bounds);
			}
		}
	};

#if !WITH_EDITOR
	// Use pre-built texture streaming data is possible
	if (CanUseTextureStreamingBuiltData() && (ComponentBuildData || bIsComponentBuildDataValid))
	{
		TArray<UTexture*, TInlineAllocator<64>> StreamableTextures;
		// bIsComponentBuildDataValid can be true, but no streamable textures were found when building texture streaming data
		if (ComponentBuildData)
		{
			for (const FStreamingTextureBuildInfo& BuildInfo : *ComponentBuildData)
			{
				if (ensure(LevelStreamingTextures.IsValidIndex(BuildInfo.TextureLevelIndex)))
				{
					if (UTexture* Texture = LevelStreamingTextures[BuildInfo.TextureLevelIndex])
					{
						check(Texture->LevelIndex != INDEX_NONE);
						if (Texture->IsStreamable())
						{
							StreamableTextures.Add(Texture);
						}
					}
				}
			}
		}

		for (UTexture* Texture : StreamableTextures)
		{
			if (int32* BuildDataIndex = GetBuildDataIndexRef(Texture))
			{
				if (*BuildDataIndex != INDEX_NONE)
				{
					FStreamingRenderAssetPrimitiveInfo& Info = OutStreamingTextures.AddDefaulted_GetRef();
					const FStreamingTextureBuildInfo& BuildInfo = (*ComponentBuildData)[*BuildDataIndex];

					Info.RenderAsset = Texture;
					Info.TexelFactor = BuildInfo.TexelFactor * ComponentScaling;
					Info.PackedRelativeBox = bUseRelativeBoxes ? BuildInfo.PackedRelativeBox : PackedRelativeBox_Identity;
					UnpackRelativeBox(ComponentBounds, Info.PackedRelativeBox, Info.Bounds);

					// Indicate that this texture build data has already been processed.
					// The build data use the merged results of all material so it only needs to be processed once.
					*BuildDataIndex = INDEX_NONE;
				}
			}
			else
			{
				// Otherwise create an entry using the available data.
				ProcessTexture(Texture);
			}
		}
	}
	else
	{
#endif
		if (TSharedPtr<const FMaterialCachedTexturesSamplingInfo> CachedSamplingInfo = MaterialData.Material->GetCachedTexturesSamplingInfo())
		{
			CachedSamplingInfo->GetStreamingRenderAssetPrimitiveInfo(ComponentBounds, ComponentScaling, *MaterialData.UVChannelData, QualityLevel, OutStreamingTextures);
		}
		// This is a legacy code path that is not expected to run on worker threads.
		else if (IsInGameThread() || IsInParallelGameThread())
		{
			TArray<UTexture*> Textures;
			MaterialData.Material->GetUsedTextures(Textures, QualityLevel, GetFeatureLevelShaderPlatform_Checked(FeatureLevel));
			for (UTexture* Texture : Textures)
			{
				ProcessTexture(Texture);
			}
		}
#if !WITH_EDITOR
	}
#endif
}

void CheckTextureStreamingBuildValidity(UWorld* InWorld)
{
#if !UE_BUILD_SHIPPING
	if (!InWorld)
	{
		return;
	}

	InWorld->NumTextureStreamingUnbuiltComponents = 0;
	InWorld->NumTextureStreamingDirtyResources = 0;

	if (CVarStreamingCheckBuildStatus.GetValueOnAnyThread() > 0)
	{
		for (int32 LevelIndex = 0; LevelIndex < InWorld->GetNumLevels(); LevelIndex++)
		{
			ULevel* Level = InWorld->GetLevel(LevelIndex);
			if (!Level)
			{
				continue;
			}

#if WITH_EDITORONLY_DATA // Only rebuild the data in editor 
			if (FPlatformProperties::HasEditorOnlyData())
			{
				TSet<FGuid> ResourceGuids;
				Level->NumTextureStreamingUnbuiltComponents = 0;

				for (AActor* Actor : Level->Actors)
				{
					// Check the actor after incrementing the progress.
					if (!Actor)
					{
						continue;
					}

					TInlineComponentArray<UPrimitiveComponent*> Primitives;
					Actor->GetComponents(Primitives);

					for (UPrimitiveComponent* Primitive : Primitives)
					{
						// Non transactional primitives like blueprints, can not invalidate the texture build for now.
						if (!Primitive || !Primitive->HasAnyFlags(RF_Transactional))
						{
							continue;
						}

						// Quality and feature level irrelevant in validation.
						if (!Primitive->BuildTextureStreamingData(TSB_ValidationOnly, EMaterialQualityLevel::Num, ERHIFeatureLevel::Num, ResourceGuids))
						{
							++Level->NumTextureStreamingUnbuiltComponents;
						}
					}
				}

				for (const FGuid& Guid : Level->TextureStreamingResourceGuids)
				{
					// If some Guid does not exists anymore, that means the resource changed.
					if (!ResourceGuids.Contains(Guid))
					{
						Level->NumTextureStreamingDirtyResources += 1;
					}
					ResourceGuids.Add(Guid);
				}

				// Don't mark package dirty as we avoid marking package dirty unless user changes something.
			}
#endif
			InWorld->NumTextureStreamingUnbuiltComponents += Level->NumTextureStreamingUnbuiltComponents;
			InWorld->NumTextureStreamingDirtyResources += Level->NumTextureStreamingDirtyResources;
		}
	}
#endif // !UE_BUILD_SHIPPING
}
