// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
TextureStreamingHelpers.cpp: Definitions of classes used for texture streaming.
=============================================================================*/

#include "Streaming/TextureStreamingHelpers.h"
#include "Stats/StatsTrace.h"
#include "UnrealEngine.h"
#include "Engine/Level.h"
#include "GenericPlatform/GenericPlatformMemoryPoolStats.h"
#include "RHI.h"

/** Streaming stats */

DECLARE_MEMORY_STAT_POOL(TEXT("Safety Pool"), STAT_Streaming01_SafetyPool, STATGROUP_Streaming, FPlatformMemory::MCR_TexturePool);
DECLARE_MEMORY_STAT_POOL(TEXT("Temporary Pool"), STAT_Streaming02_TemporaryPool, STATGROUP_Streaming, FPlatformMemory::MCR_TexturePool);
DECLARE_MEMORY_STAT_POOL(TEXT("Streaming Pool"), STAT_Streaming03_StreamingPool, STATGROUP_Streaming, FPlatformMemory::MCR_TexturePool);
DECLARE_MEMORY_STAT_POOL(TEXT("NonStreaming Mips"), STAT_Streaming04_NonStreamingMips, STATGROUP_Streaming, FPlatformMemory::MCR_TexturePool);

DECLARE_MEMORY_STAT_POOL(TEXT("Required Pool"), STAT_Streaming05_RequiredPool, STATGROUP_Streaming, FPlatformMemory::MCR_StreamingPool);
DECLARE_MEMORY_STAT_POOL(TEXT("Visible Mips"), STAT_Streaming06_VisibleMips, STATGROUP_Streaming, FPlatformMemory::MCR_StreamingPool);
DECLARE_MEMORY_STAT_POOL(TEXT("Hidden Mips"), STAT_Streaming07_HiddenMips, STATGROUP_Streaming, FPlatformMemory::MCR_StreamingPool);
DECLARE_MEMORY_STAT_POOL(TEXT("Forced Mips"), STAT_Streaming08_ForcedMips, STATGROUP_Streaming, FPlatformMemory::MCR_StreamingPool);
DECLARE_MEMORY_STAT_POOL(TEXT("UnkownRef Mips"), STAT_Streaming09_UnkownRefMips, STATGROUP_Streaming, FPlatformMemory::MCR_StreamingPool);
DECLARE_MEMORY_STAT_POOL(TEXT("Cached Mips"), STAT_Streaming11_CachedMips, STATGROUP_Streaming, FPlatformMemory::MCR_StreamingPool);

DECLARE_MEMORY_STAT_POOL(TEXT("Wanted Mips"), STAT_Streaming12_WantedMips, STATGROUP_Streaming, FPlatformMemory::MCR_UsedStreamingPool);
DECLARE_MEMORY_STAT_POOL(TEXT("Inflight Requests"), STAT_Streaming13_InflightRequests, STATGROUP_Streaming, FPlatformMemory::MCR_UsedStreamingPool);
DECLARE_MEMORY_STAT_POOL(TEXT("IO Bandwidth"), STAT_Streaming14_MipIOBandwidth, STATGROUP_Streaming, FPlatformMemory::MCR_UsedStreamingPool);
#if WITH_EDITOR
DECLARE_MEMORY_STAT_POOL(TEXT("Editor Required Pool"), STAT_Streaming15_EditorRequiredPool, STATGROUP_Streaming, FPlatformMemory::MCR_StreamingPool);
#endif
DECLARE_MEMORY_STAT_POOL(TEXT("Desired Data Loaded Percent"), STAT_Streaming16_DesiredDataLoadedPct, STATGROUP_Streaming, FPlatformMemory::MCR_UsedStreamingPool);

DECLARE_MEMORY_STAT(TEXT("Virtual Texture Pool"), STAT_Streaming17_VirtualTexturePool, STATGROUP_Streaming);
DECLARE_MEMORY_STAT(TEXT("Virtual Texture Required"), STAT_Streaming18_VirtualTextureRequired, STATGROUP_Streaming);
DECLARE_MEMORY_STAT(TEXT("Virtual Texture Used"), STAT_Streaming19_VirtualTextureUsed, STATGROUP_Streaming);

DECLARE_CYCLE_STAT(TEXT("Setup Async Task"), STAT_Streaming01_SetupAsyncTask, STATGROUP_Streaming);
DECLARE_CYCLE_STAT(TEXT("Update Streaming Data"), STAT_Streaming02_UpdateStreamingData, STATGROUP_Streaming);
DECLARE_CYCLE_STAT(TEXT("Streaming Render Assets"), STAT_Streaming03_StreamRenderAssets, STATGROUP_Streaming);
DECLARE_CYCLE_STAT(TEXT("Notifications"), STAT_Streaming04_Notifications, STATGROUP_Streaming);
DECLARE_DWORD_COUNTER_STAT(TEXT("Pending 2D Update"), STAT_Streaming05_Pending2DUpdate, STATGROUP_Streaming);
DECLARE_DWORD_COUNTER_STAT(TEXT("Virtual Texture Count"), STAT_Streaming06_VirtualTextureCount, STATGROUP_Streaming);

/** Streaming Overview stats */

DECLARE_MEMORY_STAT_POOL(TEXT("Streamable Render Assets"), STAT_StreamingOverview01_StreamableRenderAssets, STATGROUP_StreamingOverview, FPlatformMemory::MCR_StreamingPool);
DECLARE_MEMORY_STAT_POOL(TEXT("     Required"), STAT_StreamingOverview02_Required, STATGROUP_StreamingOverview, FPlatformMemory::MCR_StreamingPool);
DECLARE_MEMORY_STAT_POOL(TEXT("     Cached"), STAT_StreamingOverview03_Cached, STATGROUP_StreamingOverview, FPlatformMemory::MCR_StreamingPool);
DECLARE_MEMORY_STAT_POOL(TEXT("Streaming Overbudget"),	STAT_StreamingOverview04_StreamingOverbudget, STATGROUP_StreamingOverview, FPlatformMemory::MCR_StreamingPool);
DECLARE_MEMORY_STAT_POOL(TEXT("Unstreambale Render Assets"), STAT_StreamingOverview05_UnstreamableRenderAssets, STATGROUP_StreamingOverview, FPlatformMemory::MCR_StreamingPool);
DECLARE_MEMORY_STAT_POOL(TEXT("     NeverStream"), STAT_StreamingOverview06_NeverStream, STATGROUP_StreamingOverview, FPlatformMemory::MCR_StreamingPool);
DECLARE_MEMORY_STAT_POOL(TEXT("     UI Group"), STAT_StreamingOverview07_UIGroup, STATGROUP_StreamingOverview, FPlatformMemory::MCR_StreamingPool);
DECLARE_MEMORY_STAT_POOL(TEXT("Average Required PoolSize"),	STAT_StreamingOverview08_AverageRequiredPool, STATGROUP_StreamingOverview, FPlatformMemory::MCR_StreamingPool);

DECLARE_DWORD_COUNTER_STAT(TEXT("Number of Streamed Meshes"), STAT_StreamingOverview09_NumStreamedMeshes, STATGROUP_StreamingOverview);
DECLARE_FLOAT_COUNTER_STAT(TEXT("Average Number of Streamed Mesh LODs"), STAT_StreamingOverview10_AvgNumStreamedLODs, STATGROUP_StreamingOverview);
DECLARE_FLOAT_COUNTER_STAT(TEXT("Average Number of Resident Mesh LODs"), STAT_StreamingOverview11_AvgNumResidentLODs, STATGROUP_StreamingOverview);
DECLARE_FLOAT_COUNTER_STAT(TEXT("Average Number of Evicted Mesh LODs"), STAT_StreamingOverview12_AvgNumEvictedLODs, STATGROUP_StreamingOverview);
DECLARE_MEMORY_STAT_POOL(TEXT("Mesh LOD Bytes Streamable"), STAT_StreamingOverview13_StreamedMeshMem, STATGROUP_StreamingOverview, FPlatformMemory::MCR_StreamingPool);
DECLARE_MEMORY_STAT_POOL(TEXT("Mesh LOD Bytes Resident"), STAT_StreamingOverview14_ResidentMeshMem, STATGROUP_StreamingOverview, FPlatformMemory::MCR_StreamingPool);
DECLARE_MEMORY_STAT_POOL(TEXT("Mesh LOD Bytes Evicted"), STAT_StreamingOverview15_EvictedMeshMem, STATGROUP_StreamingOverview, FPlatformMemory::MCR_StreamingPool);

DEFINE_STAT(STAT_RenderAssetStreaming_GameThreadUpdateTime);

DEFINE_LOG_CATEGORY(LogContentStreaming);

int32 FRenderAssetStreamingSettings::ExtraIOLatency = 0;
int32 FRenderAssetStreamingSettings::ExtraIOLatencyRayTracing = 0;

ENGINE_API TAutoConsoleVariable<int32> CVarStreamingUseNewMetrics(
	TEXT("r.Streaming.UseNewMetrics"),
	1,
	TEXT("If non-zero, will use improved set of metrics and heuristics."),
	ECVF_Default);

TAutoConsoleVariable<float> CVarStreamingBoost(
	TEXT("r.Streaming.Boost"),
	1.0f,
	TEXT("=1.0: normal\n")
	TEXT("<1.0: decrease wanted mip levels\n")
	TEXT(">1.0: increase wanted mip levels"),
	ECVF_Scalability | ECVF_ExcludeFromPreview
	);

TAutoConsoleVariable<float> CVarStreamingMinBoost(
	TEXT("r.Streaming.MinBoost"),
	0.0f,
	TEXT("Minimum clamp for r.Streaming.Boost"),
	ECVF_Default
	);

TAutoConsoleVariable<float> CVarStreamingScreenSizeEffectiveMax(
	TEXT("r.Streaming.MaxEffectiveScreenSize"),
	0,
	TEXT("0: Use current actual vertical screen size\n")	
	TEXT("> 0: Clamp wanted mip size calculation to this value for the vertical screen size component."),
	ECVF_Scalability
	);

#if PLATFORM_SUPPORTS_TEXTURE_STREAMING
TAutoConsoleVariable<int32> CVarSetTextureStreaming(
	TEXT("r.TextureStreaming"),
	1,
	TEXT("Allows to define if texture streaming is enabled, can be changed at run time.\n")
	TEXT("0: off\n")
	TEXT("1: on (default)"),
	ECVF_Default | ECVF_RenderThreadSafe);
#endif

TAutoConsoleVariable<int32> CVarStreamingUseFixedPoolSize(
	TEXT("r.Streaming.UseFixedPoolSize"),
	0,
	TEXT("If non-zero, do not allow the pool size to change at run time."),
	ECVF_Scalability);

TAutoConsoleVariable<int32> CVarStreamingPoolSize(
	TEXT("r.Streaming.PoolSize"),
	-1,
	TEXT("-1: Default texture pool size, otherwise the size in MB"),
	ECVF_Scalability | ECVF_ExcludeFromPreview);

static TAutoConsoleVariable<int32> CVarStreamingPoolSizeForMeshes(
	TEXT("r.Streaming.PoolSizeForMeshes"),
	-1,
	TEXT("< 0: Mesh and texture share the same pool, otherwise the size of pool dedicated to meshes."),
	ECVF_Scalability | ECVF_ExcludeFromPreview);

static TAutoConsoleVariable<int32> CVarStreamingPoolSizeForVirtualTextures(
	TEXT("r.Streaming.PoolSizeForVirtualTextures"),
	0,
	TEXT("The maximum size in MB of the virtual texture pool to use for streaming using distance based streaming.")
	TEXT("0 (default) means that the virtual texture streaming is only GPU feedback driven."),
	ECVF_Scalability | ECVF_ExcludeFromPreview);

TAutoConsoleVariable<int32> CVarStreamingMaxTempMemoryAllowed(
	TEXT("r.Streaming.MaxTempMemoryAllowed"),
	50,
	TEXT("Maximum temporary memory used when streaming in or out texture mips.\n")
	TEXT("This memory contains mips used for the new updated texture.\n")
	TEXT("The value must be high enough to not be a limiting streaming speed factor.\n"),
	ECVF_Default);

TAutoConsoleVariable<int32> CVarStreamingDropMips(
	TEXT("r.Streaming.DropMips"),
	0,
	TEXT("0: Drop No Mips \n")
	TEXT("1: Drop Cached Mips\n")
	TEXT("2: Drop Cached and Hidden Mips\n")
	TEXT("3: Drop cached mips and non-inlined LODs of no-ref meshes\n")
	TEXT("4: Drop non-inlined LODs of no-ref meshes"),
	ECVF_Default);

TAutoConsoleVariable<int32> CVarStreamingHLODStrategy(
	TEXT("r.Streaming.HLODStrategy"),
	0,
	TEXT("Define the HLOD streaming strategy.\n")
	TEXT("0: stream\n")
	TEXT("1: stream only mip 0\n")
	TEXT("2: disable streaming"),
	ECVF_Default);

TAutoConsoleVariable<float> CVarStreamingPerTextureBiasViewBoostThreshold(
	TEXT("r.Streaming.PerTextureBiasViewBoostThreshold"),
	1.5,
	TEXT("Maximum view boost at which per texture bias will be increased.\n")
	TEXT("This prevents temporary small FOV from downgrading permanentely texture quality."),
	ECVF_Default
	);

TAutoConsoleVariable<float> CVarStreamingMaxHiddenPrimitiveViewBoost(
	TEXT("r.Streaming.MaxHiddenPrimitiveViewBoost"),
	1.5,
	TEXT("Maximum view boost that can affect hidden primitive.\n")
	TEXT("This prevents temporary small FOV from streaming all textures to their highest mips."),
	ECVF_Default
	);

TAutoConsoleVariable<float> CVarStreamingHiddenPrimitiveScale(
	TEXT("r.Streaming.HiddenPrimitiveScale"),
	0.5,
	TEXT("Define the resolution scale to apply when not in range.\n")
	TEXT(".5: drop one mip\n")
	TEXT("1: ignore visiblity"),
	ECVF_Default
	);

// Used for scalability (GPU memory, streaming stalls)
TAutoConsoleVariable<float> CVarStreamingMipBias(
	TEXT("r.Streaming.MipBias"),
	0.0f,
	TEXT("0..x reduce texture quality for streaming by a floating point number.\n")
	TEXT("0: use full resolution (default)\n")
	TEXT("1: drop one mip\n")
	TEXT("2: drop two mips"),
	ECVF_Scalability
	);

TAutoConsoleVariable<int32> CVarStreamingUsePerTextureBias(
	TEXT("r.Streaming.UsePerTextureBias"),
	1,
	TEXT("If non-zero, each texture will be assigned a mip bias between 0 and MipBias as required to fit in budget."),
	ECVF_Default);


TAutoConsoleVariable<int32> CVarStreamingFullyLoadUsedTextures(
	TEXT("r.Streaming.FullyLoadUsedTextures"),
	0,
	TEXT("If non-zero, all used texture will be fully streamed in as fast as possible"),
	ECVF_Default);

static TAutoConsoleVariable<int32> CVarStreamingFullyLoadMeshes(
	TEXT("r.Streaming.FullyLoadMeshes"),
	0,
	TEXT("If non-zero, stream in all mesh LODs. This allows semi-disabling mesh LOD streaming without recook."),
	ECVF_Default);

TAutoConsoleVariable<int32> CVarStreamingUseAllMips(
	TEXT("r.Streaming.UseAllMips"),
	0,
	TEXT("If non-zero, all available mips will be used"),
	ECVF_Default);

TAutoConsoleVariable<int32> CVarStreamingLimitPoolSizeToVRAM(
	TEXT("r.Streaming.LimitPoolSizeToVRAM"),
	0,
	TEXT("If non-zero, texture pool size with be limited to how much GPU mem is available."),
	ECVF_Scalability);

TAutoConsoleVariable<int32> CVarStreamingCheckBuildStatus(
	TEXT("r.Streaming.CheckBuildStatus"),
	0,
	TEXT("If non-zero, the engine will check whether texture streaming needs rebuild."),
	ECVF_Scalability);

TAutoConsoleVariable<int32> CVarStreamingUseMaterialData(
	TEXT("r.Streaming.UseMaterialData"),
	1,
	TEXT("If non-zero, material texture scales and coord will be used"),
	ECVF_Default);

TAutoConsoleVariable<int32> CVarStreamingNumStaticComponentsProcessedPerFrame(
	TEXT("r.Streaming.NumStaticComponentsProcessedPerFrame"),
	50,
	TEXT("If non-zero, the engine will incrementaly inserting levels by processing this amount of components per frame before they become visible"),
	ECVF_Default);

TAutoConsoleVariable<int32> CVarStreamingDefragDynamicBounds(
	TEXT("r.Streaming.DefragDynamicBounds"),
	1,
	TEXT("If non-zero, unused dynamic bounds will be removed from the update loop"),
	ECVF_Default);

// Don't split small mips as the overhead of 2 load is significant.
TAutoConsoleVariable<int32> CVarStreamingMinMipForSplitRequest(
	TEXT("r.Streaming.MinMipForSplitRequest"),
	10, // => 512
	TEXT("If non-zero, the minimum hidden mip for which load requests will first load the visible mip"),
	ECVF_Default);

TAutoConsoleVariable<float> CVarStreamingMinLevelRenderAssetScreenSize(
	TEXT("r.Streaming.MinLevelRenderAssetScreenSize"),
	100,
	TEXT("If non-zero, levels only get handled if any of their referenced texture could be required of this size. Using conservative metrics on the level data."),
	ECVF_Default);

TAutoConsoleVariable<float> CVarStreamingMaxTextureUVDensity(
	TEXT("r.Streaming.MaxTextureUVDensity"),
	0,
	TEXT("If non-zero, the max UV density a static entry can have.\n")
	TEXT("Used to improve level culling from MinLevelTextureScreenSize.\n")
	TEXT("Component with bigger entries become handled as dynamic component.\n"),
	ECVF_Default);

TAutoConsoleVariable<int32> CVarStreamingMipCalculationEnablePerLevelList(
	TEXT("r.Streaming.MipCalculationEnablePerLevelList"),
	1,
	TEXT("If non-zero, Mip size computation for streamed texture will use levels referenced with it (instead of iterating thorugh every levels).\n"),
	ECVF_Default);

ENGINE_API TAutoConsoleVariable<int32> CVarFramesForFullUpdate(
	TEXT("r.Streaming.FramesForFullUpdate"),
	5,
	TEXT("Texture streaming is time sliced per frame. This values gives the number of frames to visit all textures."));

TAutoConsoleVariable<int32> CVarStreamingLowResHandlingMode(
	TEXT("r.Streaming.LowResHandlingMode"),
	(int32)FRenderAssetStreamingSettings::LRHM_DoNothing,
	TEXT("How to handle assets with too many missing MIPs or LODs. 0 (default): do nothing, 1: load before regular streaming requests, 2: load before async loading precache requests."),
	ECVF_Default);

static TAutoConsoleVariable<int32> CVarPrioritizeMeshLODRetention(
	TEXT("r.Streaming.PrioritizeMeshLODRetention"),
	1,
	TEXT("Whether to prioritize retaining mesh LODs"),
	ECVF_Default);

static TAutoConsoleVariable<int32> CVarStreamingStressTest(
	TEXT("r.Streaming.StressTest"),
	0,
	TEXT("Set to non zero to stress test the streaming update.\n")
	TEXT("Negative values also slow down the IO.\n"),
	ECVF_Cheat);

static TAutoConsoleVariable<int32> CVarStreamingStressTestExtraIOLatency(
	TEXT("r.Streaming.StressTest.ExtraIOLatency"),
	10,
	TEXT("An extra latency in milliseconds for each stream-in request when doing the stress test."),
	ECVF_Cheat);

static TAutoConsoleVariable<int32> CVarStreamingStressTestRayTracingExtraIOLatency(
	TEXT("r.Streaming.StressTest.RayTracing.ExtraIOLatency"),
	0,
	TEXT("An extra latency in milliseconds for each ray tracing stream-in request when doing the stress test."),
	ECVF_Cheat);

static TAutoConsoleVariable<int32> CVarStreamingStressTestFramesForFullUpdate(
	TEXT("r.Streaming.StressTest.FramesForFullUpdate"),
	1,
	TEXT("Num frames to update texture states when doing the stress tests."),
	ECVF_Cheat);

static TAutoConsoleVariable<int32> CVarStreamingVRAMPercentageClamp(
	TEXT("r.Streaming.PoolSize.VRAMPercentageClamp"),
	1024,
	TEXT("When using PoolSizeVRAMPercentage, a maximum amout of memory to reserve in MB.\n")
	TEXT("This avoids reserving too much space for systems with a lot of VRAM. (default=1024)"),
	ECVF_Cheat);

void FRenderAssetStreamingSettings::Update()
{
	MaxEffectiveScreenSize = CVarStreamingScreenSizeEffectiveMax.GetValueOnAnyThread();
	MaxTempMemoryAllowed = CVarStreamingMaxTempMemoryAllowed.GetValueOnAnyThread();
	HLODStrategy = CVarStreamingHLODStrategy.GetValueOnAnyThread();
	GlobalMipBias = !GIsEditor ? FMath::FloorToInt(FMath::Max<float>(0.f, CVarStreamingMipBias.GetValueOnAnyThread())) : 0;
	PoolSize = CVarStreamingPoolSize.GetValueOnAnyThread();
	MeshPoolSize = CVarStreamingPoolSizeForMeshes.GetValueOnAnyThread();
	VirtualTexturePoolSize = CVarStreamingPoolSizeForVirtualTextures.GetValueOnAnyThread();
	bUsePerTextureBias = CVarStreamingUsePerTextureBias.GetValueOnAnyThread() != 0;
	bUseNewMetrics = CVarStreamingUseNewMetrics.GetValueOnAnyThread() != 0;
	bLimitPoolSizeToVRAM = !GIsEditor && CVarStreamingLimitPoolSizeToVRAM.GetValueOnAnyThread() != 0;
	bFullyLoadUsedTextures = CVarStreamingFullyLoadUsedTextures.GetValueOnAnyThread() != 0;
	bFullyLoadMeshes = CVarStreamingFullyLoadMeshes.GetValueOnAnyThread() != 0;
	bUseAllMips = CVarStreamingUseAllMips.GetValueOnAnyThread() != 0;
	MinMipForSplitRequest = CVarStreamingMinMipForSplitRequest.GetValueOnAnyThread();
	PerTextureBiasViewBoostThreshold = CVarStreamingPerTextureBiasViewBoostThreshold.GetValueOnAnyThread();
	MaxHiddenPrimitiveViewBoost = FMath::Max<float>(1.f, CVarStreamingMaxHiddenPrimitiveViewBoost.GetValueOnAnyThread());
	MinLevelRenderAssetScreenSize = CVarStreamingMinLevelRenderAssetScreenSize.GetValueOnAnyThread();
	MaxTextureUVDensity = CVarStreamingMaxTextureUVDensity.GetValueOnAnyThread();
	bUseMaterialData = bUseNewMetrics && CVarStreamingUseMaterialData.GetValueOnAnyThread() != 0;
	HiddenPrimitiveScale = bUseNewMetrics ? CVarStreamingHiddenPrimitiveScale.GetValueOnAnyThread() : 1.f;
	LowResHandlingMode = (ELowResHandlingMode)CVarStreamingLowResHandlingMode.GetValueOnAnyThread();
	bMipCalculationEnablePerLevelList = CVarStreamingMipCalculationEnablePerLevelList.GetValueOnAnyThread() != 0;
	bPrioritizeMeshLODRetention = CVarPrioritizeMeshLODRetention.GetValueOnAnyThread() != 0;
	VRAMPercentageClamp = CVarStreamingVRAMPercentageClamp.GetValueOnAnyThread();

	MaterialQualityLevel = (int32)GetCachedScalabilityCVars().MaterialQualityLevel;

	if (MinMipForSplitRequest <= 0)
	{
		MinMipForSplitRequest = MAX_TEXTURE_MIP_COUNT + 1;
	}

	if (bUseAllMips)
	{
		bUsePerTextureBias = false;
		GlobalMipBias = 0;
	}

#if !UE_BUILD_SHIPPING
	if (CVarStreamingStressTest.GetValueOnAnyThread() != 0)
	{
		bStressTest = true;
		// Increase threading stress between the gamethread update and the async task.
		FramesForFullUpdate = FMath::Max<int32>(CVarStreamingStressTestFramesForFullUpdate.GetValueOnAnyThread(), 0);
		// This will create cancelation requests.
		DropMips = 2; 
		// Increase chances of canceling IO while they are not yet completed.
		ExtraIOLatency = CVarStreamingStressTestExtraIOLatency.GetValueOnAnyThread();
		ExtraIOLatencyRayTracing = CVarStreamingStressTestRayTracingExtraIOLatency.GetValueOnAnyThread();
	}
    else
#endif
    {
		bStressTest = false;
		FramesForFullUpdate = FMath::Max<int32>(CVarFramesForFullUpdate.GetValueOnAnyThread(), 0);
		DropMips = CVarStreamingDropMips.GetValueOnAnyThread();
		ExtraIOLatency = 0;
		ExtraIOLatencyRayTracing = 0;
    }
}

/** the float table {-1.0f,1.0f} **/
float ENGINE_API GNegativeOneOneTable[2] = {-1.0f,1.0f};


/** Smaller value will stream out lightmaps more aggressively. */
float GLightmapStreamingFactor = 1.0f;

/** Smaller value will stream out shadowmaps more aggressively. */
float GShadowmapStreamingFactor = 0.09f;

/** For testing, finding useless textures or special demo purposes. If true, textures will never be streamed out (but they can be GC'd). 
* Caution: this only applies to unlimited texture pools (i.e. not consoles)
*/
bool GNeverStreamOutRenderAssets = false;

#if STATS
extern TAtomic<int64> GUITextureMemory;
extern TAtomic<int64> GNeverStreamTextureMemory;
extern volatile int64 GPending2DUpdateCount;

int64 GRequiredPoolSizeSum = 0;
int64 GRequiredPoolSizeCount= 0;
int64 GAverageRequiredPool = 0;
#endif

void FRenderAssetStreamingStats::Apply()
{
	const bool bHasBudgetLimit = FMath::IsWithin<int64>(RenderAssetPool, 0ll, 1ll << 40);

	/** Streaming stats */
	SET_MEMORY_STAT(MCR_TexturePool, bHasBudgetLimit ? RenderAssetPool : 0ll);
	SET_MEMORY_STAT(MCR_StreamingPool, bHasBudgetLimit ? StreamingPool : 0ll);
	SET_MEMORY_STAT(MCR_UsedStreamingPool, UsedStreamingPool);

	SET_MEMORY_STAT(STAT_Streaming01_SafetyPool, SafetyPool);
	SET_MEMORY_STAT(STAT_Streaming02_TemporaryPool, bHasBudgetLimit ? TemporaryPool : 0ll);
	SET_MEMORY_STAT(STAT_Streaming03_StreamingPool, bHasBudgetLimit ? StreamingPool : 0ll);
	SET_MEMORY_STAT(STAT_Streaming04_NonStreamingMips, NonStreamingMips);
		
	SET_MEMORY_STAT(STAT_Streaming05_RequiredPool, RequiredPool);
	SET_MEMORY_STAT(STAT_Streaming06_VisibleMips, VisibleMips);
	SET_MEMORY_STAT(STAT_Streaming07_HiddenMips, HiddenMips);
	SET_MEMORY_STAT(STAT_Streaming08_ForcedMips, ForcedMips);
	SET_MEMORY_STAT(STAT_Streaming09_UnkownRefMips, UnkownRefMips);
	SET_MEMORY_STAT(STAT_Streaming11_CachedMips, CachedMips);

	SET_MEMORY_STAT(STAT_Streaming12_WantedMips, WantedMips);
	SET_MEMORY_STAT(STAT_Streaming13_InflightRequests, PendingRequests);	
	SET_MEMORY_STAT(STAT_Streaming14_MipIOBandwidth, MipIOBandwidth);
#if WITH_EDITOR
	SET_MEMORY_STAT(STAT_Streaming15_EditorRequiredPool, RequiredPool_Editor);
#endif

	SET_MEMORY_STAT(STAT_Streaming16_DesiredDataLoadedPct, (bHasBudgetLimit && StreamingPool != 0) ? 100.0f * float(WantedMips) / float(StreamingPool) : 0.0f);
	
	SET_MEMORY_STAT(STAT_Streaming17_VirtualTexturePool, VirtualTexturePool);
	SET_MEMORY_STAT(STAT_Streaming18_VirtualTextureRequired, RequiredVirtualTexturePool);
	SET_MEMORY_STAT(STAT_Streaming19_VirtualTextureUsed, UsedVirtualTexturePool);

	SET_CYCLE_COUNTER(STAT_Streaming01_SetupAsyncTask, SetupAsyncTaskCycles);
	SET_CYCLE_COUNTER(STAT_Streaming02_UpdateStreamingData, UpdateStreamingDataCycles);
	SET_CYCLE_COUNTER(STAT_Streaming03_StreamRenderAssets, StreamRenderAssetsCycles);
	SET_CYCLE_COUNTER(STAT_Streaming04_Notifications, CallbacksCycles);
	INC_DWORD_STAT_BY(STAT_Streaming05_Pending2DUpdate, GPending2DUpdateCount);
	INC_DWORD_STAT_BY(STAT_Streaming06_VirtualTextureCount, NumVirtualTextures);

	/** Streaming Overview stats */

#if STATS
	GRequiredPoolSizeSum += RequiredPool + (GPoolSizeVRAMPercentage > 0 ? 0 : NonStreamingMips);
	GRequiredPoolSizeCount += 1;
	GAverageRequiredPool = (int64)((double)GRequiredPoolSizeSum / (double)FMath::Max<int64>(1, GRequiredPoolSizeCount));
#endif

	SET_MEMORY_STAT(STAT_StreamingOverview01_StreamableRenderAssets, RequiredPool + CachedMips); 
	SET_MEMORY_STAT(STAT_StreamingOverview02_Required, RequiredPool);
	SET_MEMORY_STAT(STAT_StreamingOverview03_Cached, CachedMips); 
	SET_MEMORY_STAT(STAT_StreamingOverview04_StreamingOverbudget, FMath::Max<int64>(RequiredPool - StreamingPool, 0)); 
	SET_MEMORY_STAT(STAT_StreamingOverview05_UnstreamableRenderAssets, NonStreamingMips);
	SET_MEMORY_STAT(STAT_StreamingOverview06_NeverStream, GNeverStreamTextureMemory.Load(EMemoryOrder::Relaxed));
	SET_MEMORY_STAT(STAT_StreamingOverview07_UIGroup, GUITextureMemory.Load(EMemoryOrder::Relaxed));
	SET_MEMORY_STAT(STAT_StreamingOverview08_AverageRequiredPool, GAverageRequiredPool); 

	SET_DWORD_STAT(STAT_StreamingOverview09_NumStreamedMeshes, NumStreamedMeshes);
	SET_FLOAT_STAT(STAT_StreamingOverview10_AvgNumStreamedLODs, AvgNumStreamedLODs);
	SET_FLOAT_STAT(STAT_StreamingOverview11_AvgNumResidentLODs, AvgNumResidentLODs);
	SET_FLOAT_STAT(STAT_StreamingOverview12_AvgNumEvictedLODs, AvgNumEvictedLODs);
	SET_MEMORY_STAT(STAT_StreamingOverview13_StreamedMeshMem, StreamedMeshMem);
	SET_MEMORY_STAT(STAT_StreamingOverview14_ResidentMeshMem, ResidentMeshMem);
	SET_MEMORY_STAT(STAT_StreamingOverview15_EvictedMeshMem, EvictedMeshMem);
}

void ResetAverageRequiredTexturePoolSize()
{
#if STATS
	GRequiredPoolSizeSum = 0;
	GRequiredPoolSizeCount= 0;
	GAverageRequiredPool = 0;
#endif
}

int64 GetAverageRequiredTexturePoolSize()
{
#if STATS
	return GAverageRequiredPool;
#else
	return 0;
#endif
}

bool OwnerLevelHasRegisteredStaticComponentsInStreamingManager(const AActor* Owner)
{
	if (Owner)
	{
		const ULevel* Level = Owner->GetLevel();
		if (Level)
		{
			return Level->bStaticComponentsRegisteredInStreamingManager;
		}
	}
	return false;
}
