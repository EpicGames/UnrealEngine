// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
TextureStreamingHelpers.h: Definitions of classes used for texture streaming.
=============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "Stats/Stats.h"
#include "HAL/IConsoleManager.h"
#include "Misc/MemStack.h"
#include "Engine/TextureDefines.h"

#ifndef STREAMING_RETRY_ON_DESERIALIZATION_ERROR
#define STREAMING_RETRY_ON_DESERIALIZATION_ERROR UE_BUILD_SHIPPING
#endif

class AActor;
class UStreamableRenderAsset;

/**
 * Streaming stats
 */

DECLARE_CYCLE_STAT_EXTERN(TEXT("Renderable Asset Streaming Game Thread Update Time"), STAT_RenderAssetStreaming_GameThreadUpdateTime,STATGROUP_Streaming, );


// Streaming Details

DECLARE_CYCLE_STAT_EXTERN(TEXT("AddToWorld Time"),STAT_AddToWorldTime,STATGROUP_StreamingDetails, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("RemoveFromWorld Time"),STAT_RemoveFromWorldTime,STATGROUP_StreamingDetails, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("UpdateLevelStreaming Time"),STAT_UpdateLevelStreamingTime,STATGROUP_StreamingDetails, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("Manage LevelsToConsider"), STAT_ManageLevelsToConsider, STATGROUP_StreamingDetails, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("Volume Streaming Tick"),STAT_VolumeStreamingTickTime,STATGROUP_StreamingDetails, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("UpdateStreamingState Time"), STAT_UpdateStreamingState, STATGROUP_StreamingDetails, );
DECLARE_DWORD_COUNTER_STAT_EXTERN(TEXT("Streaming Volumes"),STAT_VolumeStreamingChecks,STATGROUP_StreamingDetails, );


DECLARE_LOG_CATEGORY_EXTERN(LogContentStreaming, Log, All);

extern float GLightmapStreamingFactor;
extern float GShadowmapStreamingFactor;
extern bool GNeverStreamOutRenderAssets;

//@DEBUG:
// Set to 1 to log all dynamic component notifications
#define STREAMING_LOG_DYNAMIC		0
// Set to 1 to log when we change a view
#define STREAMING_LOG_VIEWCHANGES	0
// Set to 1 to log when levels are added/removed
#define STREAMING_LOG_LEVELS		0
// Set to 1 to log textures that are canceled by CancelForcedTextures()
#define STREAMING_LOG_CANCELFORCED	0

#if PLATFORM_SUPPORTS_TEXTURE_STREAMING
extern TAutoConsoleVariable<int32> CVarSetTextureStreaming;
#endif

extern TAutoConsoleVariable<float> CVarStreamingBoost;
extern TAutoConsoleVariable<float> CVarStreamingMinBoost;
extern TAutoConsoleVariable<int32> CVarStreamingUseFixedPoolSize;
extern TAutoConsoleVariable<int32> CVarStreamingPoolSize;
extern TAutoConsoleVariable<int32> CVarStreamingCheckBuildStatus;
extern TAutoConsoleVariable<int32> CVarStreamingUseMaterialData;
extern TAutoConsoleVariable<int32> CVarStreamingNumStaticComponentsProcessedPerFrame;
extern TAutoConsoleVariable<int32> CVarStreamingDefragDynamicBounds;
extern TAutoConsoleVariable<float> CVarStreamingMaxTextureUVDensity;
extern TAutoConsoleVariable<int32> CVarStreamingLowResHandlingMode;

struct FRenderAssetStreamingSettings
{
	// How to handle assets with too many missing MIPs or LODs
	enum ELowResHandlingMode
	{
		LRHM_DoNothing,
		LRHM_LoadBeforeRegular,			// Use higher IO priority than regular streaming requests
		LRHM_LoadBeforeAsyncPrecache,	// Also ensure that priority is higher than async loading precache requests
	};

	FRenderAssetStreamingSettings()
	{
		// Make sure padding bytes don't have random values
		FMemory::Memset(this, 0, sizeof(FRenderAssetStreamingSettings));
		Update();
	}

	void Update();

	FORCEINLINE bool operator ==(const FRenderAssetStreamingSettings& Rhs) const { return FMemory::Memcmp(this, &Rhs, sizeof(FRenderAssetStreamingSettings)) == 0; }
	FORCEINLINE bool operator !=(const FRenderAssetStreamingSettings& Rhs) const { return FMemory::Memcmp(this, &Rhs, sizeof(FRenderAssetStreamingSettings)) != 0; }


	float MaxEffectiveScreenSize;
	int32 MaxTempMemoryAllowed;
	int32 DropMips;
	int32 HLODStrategy;
	float HiddenPrimitiveScale;
	float PerTextureBiasViewBoostThreshold;
	float MaxHiddenPrimitiveViewBoost;
	int32 GlobalMipBias;
	int32 PoolSize;
	int32 MeshPoolSize;
	int32 VirtualTexturePoolSize;
	bool bLimitPoolSizeToVRAM;
	bool bUseNewMetrics;
	bool bFullyLoadUsedTextures;
	bool bFullyLoadMeshes;
	bool bUseAllMips;
	bool bUsePerTextureBias;
	bool bUseMaterialData;
	int32 MinMipForSplitRequest;
	float MinLevelRenderAssetScreenSize;
	float MaxTextureUVDensity;
	int32 MaterialQualityLevel;
	int32 FramesForFullUpdate;
	ELowResHandlingMode LowResHandlingMode;
	bool bMipCalculationEnablePerLevelList;
	bool bPrioritizeMeshLODRetention;
	int32 VRAMPercentageClamp;

	bool bStressTest;
	static int32 ExtraIOLatency;
	static int32 ExtraIOLatencyRayTracing;

	// Cached values of 
	bool HighPriorityLoad_Texture[TEXTUREGROUP_MAX];

protected:

};

typedef TArray<int32, TMemStackAllocator<> > FStreamingRequests;
typedef TArray<const UStreamableRenderAsset*, TInlineAllocator<12> > FRemovedRenderAssetArray;

#define NUM_BANDWIDTHSAMPLES 512
#define NUM_LATENCYSAMPLES 512

/** Streaming priority: Linear distance factor from 0 to MAX_STREAMINGDISTANCE. */
#define MAX_STREAMINGDISTANCE	10000.0f
#define MAX_MIPDELTA			5.0f
#define MAX_LASTRENDERTIME		90.0f

#ifndef UE_WITH_RENDER_ASSET_STREAMING_CSV_STATS
#define UE_WITH_RENDER_ASSET_STREAMING_CSV_STATS UE_BUILD_TEST
#endif

class UPrimitiveComponent;
class FDynamicRenderAssetInstanceManager;
template<typename T>
class FAsyncTask;
class FRenderAssetStreamingMipCalcTask;


struct FStreamingRenderAsset;
struct FStreamingContext;
struct FStreamingHandlerTextureBase;
struct FTexturePriority;

struct FRenderAssetStreamingStats
{
	FRenderAssetStreamingStats() { Reset(); }

	void Reset() { FMemory::Memzero(this, sizeof(FRenderAssetStreamingStats)); }

	void Apply();

	int64 RenderAssetPool;		// Streaming pool size in bytes given by RHI, usally some percentage of available VRAM
	// int64 StreamingPool;
	int64 UsedStreamingPool;	// Estimated memory in bytes the streamer actually use

	int64 SafetyPool;			// Memory margin in bytes we want to left unused
	int64 TemporaryPool;		// Pool size in bytes for staging resources
	int64 StreamingPool;		// Estimated memory in bytes available for streaming after subtracting non-streamable memory and safety pool
	int64 NonStreamingMips;		// Estimated memory in bytes the streamer deemed non-streamable

	int64 RequiredPool;			// Estimated memory in bytes the streamer would use if there was no limit
#if WITH_EDITOR
	int64 RequiredPool_Editor;	// Estimated memory in bytes used by the Editor
#endif
	int64 VisibleMips;			// Estimated memory in bytes used by visible assets
	int64 HiddenMips;			// Estimated memory in bytes used by hidden (e.g. culled by view frustum or occlusion) assets
	int64 ForcedMips;			// Estimated memory in bytes used by forced-fully-load assets
	int64 UnkownRefMips;		// Estimated memory in bytes used by unknown-ref-heuristic assets
	int64 CachedMips;			// Estimated memory in bytes used by the streamer for caching

	int64 WantedMips;			// Estimated memory in bytes would be resident if there was no caching
	int64 NewRequests;			// Estimated memory in bytes required by new requests (TODO)
	int64 PendingRequests;		// Estimated memory in bytes waiting to be loaded for previous requests
	int64 MipIOBandwidth;		// Estimated IO bandwidth in bytes/sec

	int64 OverBudget;			// RequiredPool - StreamingPool

	double Timestamp;			// Last time point these stats are updated in seconds

	
	volatile int32 CallbacksCycles;
	int32 SetupAsyncTaskCycles;
	int32 UpdateStreamingDataCycles;
	int32 StreamRenderAssetsCycles; // CPU cycles used to process Mip/LOD load and unload requests

	int32 NumStreamedMeshes;	// Number of meshes managed by the streamer
	float AvgNumStreamedLODs;	// Average number of mesh LODs that can be streamed
	float AvgNumResidentLODs;	// Average number of mesh LODs resident
	float AvgNumEvictedLODs;	// Average number of mesh LODs evicted
	int64 StreamedMeshMem;		// Total memory in bytes of mesh LODs that can be streamed
	int64 ResidentMeshMem;		// Total memory in bytes of resident mesh LODs
	int64 EvictedMeshMem;		// Total memory in bytes of evicted mesh LODs

	int64 VirtualTexturePool;			// Pool size used for virtual texture budget
	int32 NumVirtualTextures;			// Number of virtual textures managed by the streamer
	int64 RequiredVirtualTexturePool;	// Estimated memory in bytes the streamer would use if there was no limit
	int64 UsedVirtualTexturePool;		// Estimated memory in bytes the streamer actually use
};

// Helper to access the level bStaticComponentsRegisteredInStreamingManager flag.
extern bool OwnerLevelHasRegisteredStaticComponentsInStreamingManager(const class AActor* Owner);
