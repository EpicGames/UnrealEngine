// Copyright Epic Games, Inc. All Rights Reserved.

#include "UVChannelDensity.h"

#if WITH_EDITORONLY_DATA

#include "MeshDescription.h"
#include "StaticMeshAttributes.h"

void FUVDensityAccumulatorSourceMesh::AccumulatePolygonGroup(
	const FStaticMeshAttributes& MeshAttributes,
	FPolygonGroupID PolygonGroupID,
	int32 NumUVChannels,
	float* OutLocalWeightedUVDensities,
	float* OutLocalWeights)
{
	if (PolygonGroupID == INDEX_NONE)
	{
		return;
	}
		
	const int32 NumPolygonGroupTris = MeshDescription->GetNumPolygonGroupTriangles(PolygonGroupID);
	if (NumPolygonGroupTris == 0)
	{
		return;
	}

	TVertexAttributesConstRef<FVector3f> VertexPositions = MeshAttributes.GetVertexPositions();
	TVertexInstanceAttributesConstRef<FVector2f> VertexInstanceUVs = MeshAttributes.GetVertexInstanceUVs();
	TTriangleAttributesConstRef<TArrayView<FVertexID>> TriangleVertexIndices = MeshAttributes.GetTriangleVertexIndices();
	TTriangleAttributesConstRef<TArrayView<FVertexInstanceID>> TriangleVertexInstanceIndices = MeshAttributes.GetTriangleVertexInstanceIndices();

	FUVDensityAccumulator UVDensityAccs[MaxUVChannels];
	for (int32 UVChannelIndex = 0; UVChannelIndex < NumUVChannels; ++UVChannelIndex)
	{
		UVDensityAccs[UVChannelIndex].Reserve(NumPolygonGroupTris);
	}

	for (FTriangleID TriangleID : MeshDescription->GetPolygonGroupTriangles(PolygonGroupID))
	{
		TArrayView<const FVertexID> TriangleVerts = TriangleVertexIndices[TriangleID];
		check(TriangleVerts.Num() >= 3);

		const float Area = FUVDensityAccumulator::GetTriangleArea(
			VertexPositions[TriangleVerts[0]],
			VertexPositions[TriangleVerts[1]],
			VertexPositions[TriangleVerts[2]]);
		if (Area > UE_SMALL_NUMBER)
		{
			TArrayView<const FVertexInstanceID> TriangleVertexInstances = TriangleVertexInstanceIndices[TriangleID];
			check(TriangleVertexInstances.Num() >= 3);

			for (int32 UVChannelIndex = 0; UVChannelIndex < NumUVChannels; ++UVChannelIndex)
			{
				const float UVArea = FUVDensityAccumulator::GetUVChannelArea(
					VertexInstanceUVs.Get(TriangleVertexInstances[0], UVChannelIndex),
					VertexInstanceUVs.Get(TriangleVertexInstances[1], UVChannelIndex),
					VertexInstanceUVs.Get(TriangleVertexInstances[2], UVChannelIndex));
				UVDensityAccs[UVChannelIndex].PushTriangle(Area, UVArea);
			}
		}
	}

	for (int32 UVChannelIndex = 0; UVChannelIndex < NumUVChannels; ++UVChannelIndex)
	{
		UVDensityAccs[UVChannelIndex].AccumulateDensity(OutLocalWeightedUVDensities[UVChannelIndex], OutLocalWeights[UVChannelIndex]);
	}
}

void FUVDensityAccumulatorSourceMesh::FinalAccumulate(
	int32 NumUVChannels,
	const float* LocalWeightedUVDensities,
	const float* LocalWeights,
	TArrayView<float> OutWeightedUVDensities,
	TArrayView<float> OutWeights)
{
	if (InstanceScales.Num() == 0) // Single instance
	{
		for (int32 i = 0; i < NumUVChannels; ++i)
		{
			OutWeightedUVDensities[i] += LocalWeightedUVDensities[i];
			OutWeights[i] += LocalWeights[i];
		}
	}
	else
	{
		// For instances (assemblies), accumulate contribution from each instance to the output
		for (float InstanceScale : InstanceScales)
		{
			for (int32 i = 0; i < NumUVChannels; ++i)
			{
				OutWeightedUVDensities[i] += LocalWeightedUVDensities[i] * InstanceScale * InstanceScale;
				OutWeights[i] += LocalWeights[i] * InstanceScale;
			}
		}
	}
}

bool FUVDensityAccumulatorSourceMesh::AccumulateDensitiesForMaterial(
	FName MaterialSlotName,
	TArrayView<float> OutWeightedUVDensities,
	TArrayView<float> OutWeights)
{
	check(MeshDescription != nullptr);
	check(OutWeightedUVDensities.Num() == OutWeights.Num());
	check(OutWeightedUVDensities.Num() <= MaxUVChannels);

	const int32 NumUVChannels = FMath::Min3(MeshDescription->GetNumUVElementChannels(), OutWeightedUVDensities.Num(), MaxUVChannels);
	float LocalWeightedUVDensities[MaxUVChannels] {};
	float LocalWeights[MaxUVChannels] {};

	const FStaticMeshAttributes MeshAttributes(*MeshDescription);
	TPolygonGroupAttributesConstRef<FName> PolygonGroupMaterialSlotNames = MeshAttributes.GetPolygonGroupMaterialSlotNames();
	bool bAnyPolygonGroups = false;
	for (FPolygonGroupID PolygonGroupID : MeshDescription->PolygonGroups().GetElementIDs())
	{
		if (MaterialSlotName == PolygonGroupMaterialSlotNames[PolygonGroupID])
		{
			AccumulatePolygonGroup(MeshAttributes, PolygonGroupID, NumUVChannels, LocalWeightedUVDensities, LocalWeights);
			bAnyPolygonGroups = true;
		}
	}

	if (bAnyPolygonGroups)
	{
		FinalAccumulate(NumUVChannels, LocalWeightedUVDensities, LocalWeights, OutWeightedUVDensities, OutWeights);
	}

	return bAnyPolygonGroups;
}

#endif // WITH_EDITORONLY_DATA