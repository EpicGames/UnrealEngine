// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
UVChannelDensity.h: Helpers to compute UV channel density.
=============================================================================*/

#pragma once

#include "CoreMinimal.h"

#if WITH_EDITORONLY_DATA

#include "MeshTypes.h"

struct FMeshDescription;
class FStaticMeshAttributes;

struct FUVDensityAccumulator
{
private:

	struct FElementInfo
	{
		float Weight;
		float UVDensity;
		FElementInfo(float InWeight, float InUVDensity) : Weight(InWeight), UVDensity(InUVDensity) {}
	};

	TArray<FElementInfo> Elements;

public:

	void Reserve(int32 Size) { Elements.Reserve(Size); }

	void PushTriangle(float InArea, float InUVArea)
	{
		if (InArea > UE_SMALL_NUMBER && InUVArea > UE_SMALL_NUMBER)
		{
			Elements.Add(FElementInfo(FMath::Sqrt(InArea), FMath::Sqrt(InArea / InUVArea)));
		}
	}

	void AccumulateDensity(float& WeightedUVDensity, float& Weight, float DiscardPercentage = .10f)
	{
		struct FCompareUVDensity
		{
			FORCEINLINE bool operator()(FElementInfo const& A, FElementInfo const& B) const { return A.UVDensity < B.UVDensity; }
		};

		Elements.Sort(FCompareUVDensity());

		// Remove 10% of higher and lower texel factors.
		const int32 Threshold = FMath::FloorToInt(DiscardPercentage * (float)Elements.Num());
		for (int32 Index = Threshold; Index < Elements.Num() - Threshold; ++Index)
		{
			const FElementInfo& Element = Elements[Index];
			WeightedUVDensity += Element.UVDensity * Element.Weight;
			Weight += Element.Weight;
		}
	}

	float GetDensity(float DiscardPercentage = .10f)
	{
		float WeightedUVDensity = 0;
		float Weight = 0;

		AccumulateDensity(WeightedUVDensity, Weight, DiscardPercentage);

		return (Weight > UE_SMALL_NUMBER) ? (WeightedUVDensity / Weight) : 0;
	}

	FORCEINLINE_DEBUGGABLE 
	static float GetTriangleArea(const FVector3f& Pos0, const FVector3f& Pos1, const FVector3f& Pos2)
	{
		FVector3f P01 = Pos1 - Pos0;
		FVector3f P02 = Pos2 - Pos0;
		return FVector3f::CrossProduct(P01, P02).Size();
	}

	FORCEINLINE_DEBUGGABLE 
	static float GetUVChannelArea(const FVector2f& UV0, const FVector2f& UV1, const FVector2f& UV2)
	{
		FVector2f UV01 = UV1 - UV0;
		FVector2f UV02 = UV2 - UV0;
		return FMath::Abs<float>(UV01.X * UV02.Y - UV01.Y * UV02.X);
	}
};

struct FUVDensityAccumulatorSourceMesh
{
public:
	static const int32 MaxUVChannels = 8;

	FUVDensityAccumulatorSourceMesh(
		FMeshDescription& InMeshDescription,
		TArray<float>&& InInstanceScales = TArray<float>())
	: MeshDescription(&InMeshDescription)
	, InstanceScales(InInstanceScales)
	{
	}

	bool AccumulateDensitiesForMaterial(FName MaterialSlotName, TArrayView<float> OutWeightedUVDensities, TArrayView<float> OutWeights);

private:
	FMeshDescription* MeshDescription;
	TArray<float> InstanceScales;

	void AccumulatePolygonGroup(
		const FStaticMeshAttributes& MeshAttributes,
		FPolygonGroupID PolygonGroupID,
		int32 NumUVChannels,
		float* OutLocalWeightedUVDensities,
		float* OutLocalWeights);

	void FinalAccumulate(
		int32 NumUVChannels,
		const float* LocalWeightedUVDensities,
		const float* LocalWeights,
		TArrayView<float> OutWeightedUVDensities,
		TArrayView<float> OutWeights);
};

#endif
