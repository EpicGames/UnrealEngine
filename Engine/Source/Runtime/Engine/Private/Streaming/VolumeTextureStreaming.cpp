// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
VolumeTextureStreaming.cpp: Helpers to stream in and out volume texture LODs.
=============================================================================*/

#include "Streaming/VolumeTextureStreaming.h"
#include "RenderingThread.h"
#include "Streaming/TextureMipAllocator.h"

//*****************************************************************************
//******************** FVolumeTextureMipAllocator_Reallocate ******************
//*****************************************************************************

FVolumeTextureMipAllocator_Reallocate::FVolumeTextureMipAllocator_Reallocate(UTexture* Texture)
	: FTextureMipAllocator(Texture, ETickState::AllocateMips, ETickThread::Async)
	, StreamedInMipData(PendingFirstLODIdx)
{
}

FVolumeTextureMipAllocator_Reallocate::~FVolumeTextureMipAllocator_Reallocate()
{
}

bool FVolumeTextureMipAllocator_Reallocate::AllocateMips(const FTextureUpdateContext& Context, FTextureMipInfoArray& OutMipInfos, const FTextureUpdateSyncOptions& SyncOptions)
{
	OutMipInfos.AddDefaulted(CurrentFirstLODIdx);

	// Allocate the mip memory as temporary buffers so that the FTextureMipDataProvider implementation can write to it.
	for (int32 MipIdx = PendingFirstLODIdx; MipIdx < CurrentFirstLODIdx; ++MipIdx)
	{
		const FTexture2DMipMap& OwnerMip = *Context.MipsView[MipIdx];
		FTextureMipInfo& MipInfo = OutMipInfos[MipIdx];

		// Note that streamed in mips size will always be at least as big as block size.
		MipInfo.Format = Context.Resource->GetPixelFormat();
		MipInfo.SizeX = OwnerMip.SizeX;
		MipInfo.SizeY = OwnerMip.SizeY;
		MipInfo.SizeZ = OwnerMip.SizeZ;

		const FRHITextureDesc Desc =
			FRHITextureCreateDesc::Create3D(TEXT("Temp"), MipInfo.SizeX, MipInfo.SizeY, MipInfo.SizeZ, MipInfo.Format)
			.SetFlags(Context.Resource->GetCreationFlags())
			.SetExtData(Context.Resource->GetExtData());

		MipInfo.DataSize = RHICalcTexturePlatformSize(Desc).Size;

		StreamedInMipData.GetMipSize()[MipIdx] = MipInfo.DataSize;

		// When initializing the texture with it's bulk data, we proceed with a single allocation.
		if (!GUseTexture3DBulkDataRHI)
		{
			MipInfo.DestData = FMemory::Malloc(MipInfo.DataSize, FVolumeTextureBulkData::MALLOC_ALIGNMENT);
			StreamedInMipData.GetMipData()[MipIdx] = MipInfo.DestData;
		}
	}

	if (GUseTexture3DBulkDataRHI)
	{
		StreamedInMipData.MergeMips(ResourceState.MaxNumLODs);

		uint8* MergedMipData = (uint8*)StreamedInMipData.GetResourceBulkData();
		for (int32 MipIdx = PendingFirstLODIdx; MipIdx < CurrentFirstLODIdx; ++MipIdx)
		{
			FTextureMipInfo& MipInfo = OutMipInfos[MipIdx];
			MipInfo.DestData = MergedMipData;
			MergedMipData += MipInfo.DataSize;
		}
	}

	AdvanceTo(ETickState::UploadMips, ETickThread::Render);
	return true;
}

bool FVolumeTextureMipAllocator_Reallocate::UploadMips(const FTextureUpdateContext& Context, const FTextureUpdateSyncOptions& SyncOptions)
{
	FRHICommandListImmediate& RHICmdList = FRHICommandListImmediate::Get();

	EPixelFormat PixelFormat = Context.Resource->GetPixelFormat();

	// Create new Texture.
	{
		const FTexture2DMipMap& FirstMip = *Context.MipsView[PendingFirstLODIdx];

		FRHITextureCreateDesc Desc =
			FRHITextureCreateDesc::Create3D(TEXT("FinalizeMips"), FirstMip.SizeX, FirstMip.SizeY, FirstMip.SizeZ, PixelFormat)
			.SetNumMips(ResourceState.NumRequestedLODs)
			.SetFlags(Context.Resource->GetCreationFlags())
			.SetExtData(Context.Resource->GetExtData());

		if (GUseTexture3DBulkDataRHI)
		{
			Desc.SetInitActionBulkData(&StreamedInMipData);
		}

		Desc.DetermineInititialState();

		IntermediateTextureRHI = RHICmdList.CreateTexture(Desc);
	}

	// Copy shared mips.
	{
		FRHITexture* SrcTexture = Context.Resource->GetTexture3DRHI();
		FRHITexture* DstTexture = IntermediateTextureRHI.GetReference();

		UE::RHI::CopySharedMips_AssumeSRVMaskState(RHICmdList, SrcTexture, DstTexture);
	}

	// Update the streamed in mips if they were not initialized from the bulk data.
	if (!GUseTexture3DBulkDataRHI)
	{
		const int32 BlockSizeX = GPixelFormats[PixelFormat].BlockSizeX;
		const int32 BlockSizeY = GPixelFormats[PixelFormat].BlockSizeY;
		const int32 BlockBytes = GPixelFormats[PixelFormat].BlockBytes;
		ensure(GPixelFormats[PixelFormat].BlockSizeZ == 1);

		for (int32 MipIdx = PendingFirstLODIdx; MipIdx < CurrentFirstLODIdx; ++MipIdx)
		{
			const FTexture2DMipMap& Mip = *Context.MipsView[MipIdx];
			const uint8* MipData = (uint8*)StreamedInMipData.GetMipData()[MipIdx];
			if (MipData)
			{
				FUpdateTextureRegion3D UpdateRegion(0, 0, 0, 0, 0, 0, Mip.SizeX, Mip.SizeY, Mip.SizeZ);
				const uint32 NumBlockX = (uint32)FMath::DivideAndRoundUp<int32>(Mip.SizeX, BlockSizeX);
				const uint32 NumBlockY = (uint32)FMath::DivideAndRoundUp<int32>(Mip.SizeY, BlockSizeY);
				RHICmdList.UpdateTexture3D(IntermediateTextureRHI, MipIdx - PendingFirstLODIdx, UpdateRegion, NumBlockX * BlockBytes, NumBlockX * NumBlockY * BlockBytes, MipData);
			}
		}
	}
	StreamedInMipData.Discard();

	AdvanceTo(ETickState::Done, ETickThread::None);
	return true;
}

void FVolumeTextureMipAllocator_Reallocate::Cancel(const FTextureUpdateSyncOptions& SyncOptions)
{
	// Release the temporary mip data. Can be run on either renderthread or async threads.
	StreamedInMipData.Discard();
}

FTextureMipAllocator::ETickThread FVolumeTextureMipAllocator_Reallocate::GetCancelThread() const
{
	// If there is an  intermediate texture, it is safer to released on the renderthread.
	if (StreamedInMipData.GetResourceBulkData())
	{
		return ETickThread::Async;
	}
	// Nothing to do.
	else
	{
		return ETickThread::None;
	}
}
