// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
VolumeTextureStreaming.h: Helpers to stream in and out volume texture LODs.
=============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "RenderAssetUpdate.h"
#include "Engine/VolumeTexture.h"
#include "Rendering/Texture3DResource.h"
#include "Texture2DMipDataProvider_DDC.h"
#include "Texture2DMipDataProvider_IO.h"
#include "TextureMipAllocator.h"

#if WITH_EDITORONLY_DATA
typedef FTexture2DMipDataProvider_DDC FVolumeTextureMipDataProvider_DDC;
#endif

typedef FTexture2DMipDataProvider_IO FVolumeTextureMipDataProvider_IO;

/**
* Implements a texture3d mip resizing strategy based on creating a duplicate texture and copyinng the shared mips.
*/
class FVolumeTextureMipAllocator_Reallocate : public FTextureMipAllocator
{
public:

	FVolumeTextureMipAllocator_Reallocate(UTexture* Texture);
	~FVolumeTextureMipAllocator_Reallocate();

	// ********************************************************
	// ********* FTextureMipAllocator implementation **********
	// ********************************************************

	bool AllocateMips(const FTextureUpdateContext& Context, FTextureMipInfoArray& OutMipInfos, const FTextureUpdateSyncOptions& SyncOptions) final override;
	bool UploadMips(const FTextureUpdateContext& Context, const FTextureUpdateSyncOptions& SyncOptions) final override;
	void Cancel(const FTextureUpdateSyncOptions& SyncOptions) final override;
	ETickThread GetCancelThread() const final override;

protected:
	// The temporary main memory allocations holding the mip data
	FVolumeTextureBulkData StreamedInMipData;
};
