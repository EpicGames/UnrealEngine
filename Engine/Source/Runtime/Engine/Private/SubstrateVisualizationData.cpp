// Copyright Epic Games, Inc. All Rights Reserved.

#include "SubstrateVisualizationData.h"
#include "HAL/IConsoleManager.h"
#include "Materials/Material.h"
#include "Misc/ConfigCacheIni.h"
#include "RenderUtils.h"

#define LOCTEXT_NAMESPACE "FSubstrateVisualizationData"

static FSubstrateVisualizationData GSubstrateVisualizationData;

static FString ConfigureConsoleCommand(FSubstrateVisualizationData::TModeMap& ModeMap, IConsoleVariable*& OutCVar)
{
	FString AvailableVisualizationModes;
	for (FSubstrateVisualizationData::TModeMap::TConstIterator It = ModeMap.CreateConstIterator(); It; ++It)
	{
		const FSubstrateVisualizationData::FModeRecord& Record = It.Value();
		AvailableVisualizationModes += FString("\n  Value=");
		AvailableVisualizationModes += FString::Printf(TEXT("%d: "), uint8(Record.ViewMode));
		AvailableVisualizationModes += Record.ModeString;
		if (!Record.bAvailableCommand)
		{
			AvailableVisualizationModes += FString::Printf(TEXT(" --- Unavailable, reason: %s"), *Record.UnavailableReason.ToString());
		}
	}

	FString Out;
	Out = TEXT("When the viewport view-mode is set to 'Substrate Visualization', this command specifies which of the various channels to display. Values entered other than the allowed values shown below will be ignored.");
	Out += AvailableVisualizationModes;

	OutCVar = IConsoleManager::Get().RegisterConsoleVariable(
		FSubstrateVisualizationData::GetVisualizeConsoleCommandName(),
		0,
		*Out,
		ECVF_Cheat);

	return Out;
}

static void AddVisualizationMode(
	FSubstrateVisualizationData::TModeMap& ModeMap,
	const TCHAR* ModeString,
	const FText& ModeText,
	const FText& ModeDesc,
	const FSubstrateViewMode ViewMode,
	bool bDefaultComposited,
	bool bAvailableCommand,
	const FText& UnavailableReason
)
{
	const FName ModeName = FName(ModeString);

	FSubstrateVisualizationData::FModeRecord& Record = ModeMap.Emplace(ModeName);
	Record.ModeString			= FString(ModeString);
	Record.ModeName				= ModeName;
	Record.ModeText				= ModeText;
	Record.ModeDesc				= ModeDesc;
	Record.ViewMode				= ViewMode;
	Record.bDefaultComposited	= bDefaultComposited;
	Record.bAvailableCommand	= bAvailableCommand;
	Record.UnavailableReason	= UnavailableReason;
}

void FSubstrateVisualizationData::Initialize()
{
	if (!bIsInitialized && Substrate::IsSubstrateEnabled())
	{
		TModeMap AllModeMap;

		AddVisualizationMode(
			AllModeMap,
			TEXT("MaterialProperties"),
			LOCTEXT("MaterialProperties", "Material Properties"),
			LOCTEXT("MaterialPropertiesDesc", "Visualizes Substrate material properties under mouse cursor"),
			FSubstrateViewMode::MaterialProperties,
			true,
			true,
			LOCTEXT("None", "None"));

		AddVisualizationMode(
			AllModeMap,
			TEXT("MaterialCount"),
			LOCTEXT("MaterialCount", "Material Count"),
			LOCTEXT("MaterialCountDesc", "Visualizes Substrate material count per pixel"),
			FSubstrateViewMode::MaterialCount,
			true,
			true,
			LOCTEXT("None", "None"));

		AddVisualizationMode(
			AllModeMap,
			TEXT("AdvancedMaterialProperties"),
			LOCTEXT("AdvancedMaterialProperties", "Advanced Material Properties"),
			LOCTEXT("AdvancedMaterialPropertiesDesc", "Visualizes Substrate advanced material properties"),
			FSubstrateViewMode::AdvancedMaterialProperties,
			true,
			Substrate::IsAdvancedVisualizationEnabled(GMaxRHIShaderPlatform),
			LOCTEXT("IsSubstrateAdvancedDebugShaderEnabled", "Substrate advanced debugging r.Substrate.Debug.AdvancedVisualizationShaders is disabled"));

		AddVisualizationMode(
			AllModeMap,
			TEXT("MaterialClassification"),
			LOCTEXT("MaterialClassification", "Material Classification"),
			LOCTEXT("MaterialClassificationDesc", "Visualizes Substrate material classification"),
			FSubstrateViewMode::MaterialClassification,
			true,
			true,
			LOCTEXT("None", "None"));

		AddVisualizationMode(
			AllModeMap,
			TEXT("DecalClassification"),
			LOCTEXT("DecalClassification", "Decal classification"),
			LOCTEXT("DecalClassificationDesc", "Visualizes Substrate decal classification"),
			FSubstrateViewMode::DecalClassification,
			true,
			false, // Disable for now, as it is not important, and is mainly used for debugging
			LOCTEXT("IsSubstrateBufferPassEnabled", "Substrate tiled DBuffer pass (r.Substrate.DBufferPass and r.Substrate.DBufferPass.DedicatedTiles) is disabled"));

		AddVisualizationMode(
			AllModeMap,
			TEXT("OpaqueRoughRefractionClassification"),
			LOCTEXT("OpaqueRoughRefractionClassification", "Opaque Rough Refraction Classification"),
			LOCTEXT("OpaqueRoughRefractionClassificationDesc", "Visualizes Substrate Opaque Rough Refraction Classification"),
			FSubstrateViewMode::RoughRefractionClassification,
			true,
			Substrate::IsOpaqueRoughRefractionEnabled(GMaxRHIShaderPlatform),
			LOCTEXT("IsSubstrateRoughRefractionEnabled", "Substrate rough refraction r.Substrate.OpaqueMaterialRoughRefraction is disabled"));

		AddVisualizationMode(
			AllModeMap,
			TEXT("SubstrateInfo"),
			LOCTEXT("SubstrateInfo", "Substrate Info"),
			LOCTEXT("SubstrateInfoDesc", "Visualizes Substrate info"),
			FSubstrateViewMode::SubstrateInfo,
			true,
			true,
			LOCTEXT("None", "None"));

		AddVisualizationMode(
			AllModeMap,
			TEXT("MaterialByteCount"),
			LOCTEXT("MaterialByteCount", "Material Bytes Count"),
			LOCTEXT("MaterialByteCountDesc", "Visualizes Substrate material footprint per pixel"),
			FSubstrateViewMode::MaterialByteCount,
			true,
			true,
			LOCTEXT("None", "None"));
		

		ConsoleDocumentationVisualizationMode = ConfigureConsoleCommand(AllModeMap, CVarViewModes);

		// Now only copy the available modes for the menu to not overload it with useless entries.
		for (auto& Mode : AllModeMap)
		{
			if(Mode.Value.bAvailableCommand)
			{
				ModeMap.Emplace(Mode.Key) = Mode.Value;
			}
		}
	}
	bIsInitialized = true;
}

FText FSubstrateVisualizationData::GetModeDisplayName(const FName& InModeName) const
{
	if (const FModeRecord* Record = ModeMap.Find(InModeName))
	{
		return Record->ModeText;
	}
	else
	{
		return FText::GetEmpty();
	}
}

FSubstrateViewMode FSubstrateVisualizationData::GetViewMode(const FName& InModeName) const
{
	if (const FModeRecord* Record = ModeMap.Find(InModeName))
	{
		return Record->ViewMode;
	}
	else
	{
		return FSubstrateViewMode::None;
	}
}

uint32 FSubstrateVisualizationData::GetViewMode()
{
	uint32 OutViewMode = 0;
	if (GSubstrateVisualizationData.IsInitialized() && GSubstrateVisualizationData.CVarViewModes && GSubstrateVisualizationData.CVarViewModes->AsVariableInt())
	{
		OutViewMode = GSubstrateVisualizationData.CVarViewModes->AsVariableInt()->GetValueOnRenderThread();
	}
	return OutViewMode;
}

bool FSubstrateVisualizationData::GetModeDefaultComposited(const FName& InModeName) const
{
	if (const FModeRecord* Record = ModeMap.Find(InModeName))
	{
		return Record->bDefaultComposited;
	}
	else
	{
		return false;
	}
}

FSubstrateVisualizationData& GetSubstrateVisualizationData()
{
	if (!GSubstrateVisualizationData.IsInitialized())
	{
		GSubstrateVisualizationData.Initialize();
	}

	return GSubstrateVisualizationData;
}

#undef LOCTEXT_NAMESPACE
