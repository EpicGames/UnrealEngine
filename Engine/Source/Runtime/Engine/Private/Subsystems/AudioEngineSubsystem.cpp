// Copyright Epic Games, Inc. All Rights Reserved.

#include "Subsystems/AudioEngineSubsystem.h"
#include "AudioDeviceManager.h"
#include "AudioMixerDevice.h"
#include "Subsystems/Subsystem.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AudioEngineSubsystem)

UAudioEngineSubsystem::UAudioEngineSubsystem()
	: UDynamicSubsystem()
{
}

Audio::FDeviceId UAudioEngineSubsystem::GetAudioDeviceId() const
{
	const UAudioSubsystemCollectionRoot* SubsystemRoot = Cast<UAudioSubsystemCollectionRoot>(GetOuter());
	check(SubsystemRoot);

	return SubsystemRoot->GetAudioDeviceID();
}

FAudioDeviceHandle UAudioEngineSubsystem::GetAudioDeviceHandle() const
{
	Audio::FDeviceId DeviceId = GetAudioDeviceId();

	if (FAudioDeviceManager* AudioDeviceManager = FAudioDeviceManager::Get())
	{
		return AudioDeviceManager->GetAudioDevice(DeviceId);
	}

	return FAudioDeviceHandle();
}

Audio::FMixerSourceManager* UAudioEngineSubsystem::GetMutableSourceManager()
{
	Audio::FMixerDevice* MixerDevice = GetMutableMixerDevice();
	if (MixerDevice)
	{
		return MixerDevice->GetSourceManager();
	}
	return nullptr;
}

const Audio::FMixerSourceManager* UAudioEngineSubsystem::GetSourceManager() const
{
	const Audio::FMixerDevice* MixerDevice = GetMixerDevice();
	if (MixerDevice)
	{
		return MixerDevice->GetSourceManager();
	}
	return nullptr;
}

Audio::FMixerDevice* UAudioEngineSubsystem::GetMutableMixerDevice()
{
	Audio::FMixerDevice* MixerDevice = static_cast<Audio::FMixerDevice*>(GetAudioDeviceHandle().GetAudioDevice());
	return MixerDevice;
}

const Audio::FMixerDevice* UAudioEngineSubsystem::GetMixerDevice() const
{
	const Audio::FMixerDevice* MixerDevice = static_cast<Audio::FMixerDevice*>(GetAudioDeviceHandle().GetAudioDevice());
	return MixerDevice;
}

