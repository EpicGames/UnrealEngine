// Copyright Epic Games, Inc. All Rights Reserved.

#include "Subsystems/SubsystemBlueprintLibrary.h"

#include "AudioDevice.h"
#include "Blueprint/UserWidget.h"
#include "Components/ActorComponent.h"
#include "Engine/Engine.h"
#include "Engine/GameInstance.h"
#include "GameFramework/Actor.h"
#include "GameFramework/Pawn.h"
#include "Subsystems/EngineSubsystem.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SubsystemBlueprintLibrary)

/*static*/  UEngineSubsystem* USubsystemBlueprintLibrary::GetEngineSubsystem(TSubclassOf<UEngineSubsystem> Class)
{
	return GEngine->GetEngineSubsystemBase(Class);
}

/*static*/ UGameInstanceSubsystem* USubsystemBlueprintLibrary::GetGameInstanceSubsystem(UObject* ContextObject, TSubclassOf<UGameInstanceSubsystem> Class)
{
	if (const UWorld* World = ThisClass::GetWorldFrom(ContextObject))
	{
		if (const UGameInstance* GameInstance = World->GetGameInstance())
		{
			return GameInstance->GetSubsystemBase(Class);
		}
	}
	return nullptr;
}

/*static*/ ULocalPlayerSubsystem* USubsystemBlueprintLibrary::GetLocalPlayerSubsystem(UObject* ContextObject, TSubclassOf<ULocalPlayerSubsystem> Class)
{
	const ULocalPlayer* LocalPlayer = nullptr;

	if (const UUserWidget* UserWidget = Cast<UUserWidget>(ContextObject))
	{
		LocalPlayer = UserWidget->GetOwningLocalPlayer();
	}
	else if (const APlayerController* PlayerController = Cast<APlayerController>(ContextObject))
	{
		LocalPlayer = Cast<ULocalPlayer>(PlayerController->Player);
	}
	else if (const APawn* Pawn = Cast<APawn>(ContextObject))
	{
		const APlayerController* PC = Cast<APlayerController>(Pawn->GetController());
		
		LocalPlayer = PC ? PC->GetLocalPlayer() : nullptr;
	}
	else if (const AActor* Actor = Cast<AActor>(ContextObject))
	{
		const APlayerController* PC = Cast<APlayerController>(Actor->GetInstigatorController());
		
		LocalPlayer = PC ? PC->GetLocalPlayer() : nullptr;
	}
	else if (const UActorComponent* ActorComponent = Cast<UActorComponent>(ContextObject))
	{
		const AActor* OwningActor = ActorComponent->GetOwner<AActor>();
		const APlayerController* PC = Cast<APlayerController>(OwningActor->GetInstigatorController());
		
		LocalPlayer = PC ? PC->GetLocalPlayer() : nullptr;
	}
	else
	{
		LocalPlayer = Cast<ULocalPlayer>(ContextObject);
	}

	if (LocalPlayer != nullptr)
	{
		return LocalPlayer->GetSubsystemBase(Class);
	}

	return nullptr;
}

UWorldSubsystem* USubsystemBlueprintLibrary::GetWorldSubsystem(UObject* ContextObject, TSubclassOf<UWorldSubsystem> Class)
{
	if (const UWorld* World = ThisClass::GetWorldFrom(ContextObject))
	{
		return World->GetSubsystemBase(Class);
	}
	return nullptr;
}

/*static*/ UAudioEngineSubsystem* USubsystemBlueprintLibrary::GetAudioEngineSubsystem(UObject* ContextObject, TSubclassOf<UAudioEngineSubsystem> Class)
{
	if (UWorld* World = ThisClass::GetWorldFrom(ContextObject))
	{
		FAudioDeviceHandle AudioDeviceHandle = World->GetAudioDevice();
		if (AudioDeviceHandle.IsValid())
		{
			return AudioDeviceHandle->GetSubsystemBase(Class);
		}
	}
	return nullptr;
}

/*static*/ ULocalPlayerSubsystem* USubsystemBlueprintLibrary::GetLocalPlayerSubSystemFromPlayerController(APlayerController* PlayerController, TSubclassOf<ULocalPlayerSubsystem> Class)
{
	if (PlayerController)
	{
		if (const ULocalPlayer* LocalPlayer = Cast<ULocalPlayer>(PlayerController->Player))
		{
			return LocalPlayer->GetSubsystemBase(Class);
		}
	}
	return nullptr;
}

/*static*/ UWorld* USubsystemBlueprintLibrary::GetWorldFrom(UObject* ContextObject)
{
	if (ContextObject)
	{
		return GEngine->GetWorldFromContextObject(ContextObject, EGetWorldErrorMode::ReturnNull);
	}
	return nullptr;
}
