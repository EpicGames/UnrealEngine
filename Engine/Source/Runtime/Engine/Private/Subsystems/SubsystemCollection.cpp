// Copyright Epic Games, Inc. All Rights Reserved.

#include "Subsystems/SubsystemCollection.h"

#include "Algo/RemoveIf.h"
#include "Subsystems/Subsystem.h"
#include "UObject/Interface.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/UObjectHash.h"
#include "UObject/Package.h"
#include "Logging/StructuredLog.h"
#include "Modules/ModuleManager.h"
#include "Misc/PackageName.h"
#include "Clang/ClangPlatformCompilerPreSetup.h"

DEFINE_LOG_CATEGORY(LogSubsystemCollection);

/** FSubsystemModuleWatcher class to hide the implementation of keeping the DynamicSystemModuleMap up to date*/
class FSubsystemModuleWatcher
{
public:
	static void OnModulesChanged(FName ModuleThatChanged, EModuleChangeReason ReasonForChange);

	/** Init / Deinit the Module watcher, this tracks module startup and shutdown to ensure only the appropriate dynamic subsystems are instantiated */
	static void InitializeModuleWatcher();
	static void DeinitializeModuleWatcher();

private:
	static void AddClassesForModule(const FName& InModuleName);
	static void RemoveClassesForModule(const FName& InModuleName);

	static FDelegateHandle ModulesChangedHandle;
};

// globals without thread protection must be accessed only from GameThread
FDelegateHandle FSubsystemModuleWatcher::ModulesChangedHandle;
static TArray<FSubsystemCollectionBase*> GlobalSubsystemCollections;
static TMap<FName, TArray<UClass*>> GlobalDynamicSystemModuleMap;

struct FSubsystemCollectionInitialization
{
	// Classes of subsystem we intend to initialize, for handling calls to GetSubsystem during initialization
	// Key is a base/interface/concrete class which may be used in a call to GetSubsystem
	// Value is the concrete class that will be returned
	// So when key == value, this is a concrete class we intend to initialize
	TMap<UClass*, UClass*> ClassMap;

	// List of classes to be initialized. Classes may be initialized before we reach them in the queue via
	// re-entrancy.
	// We also may add more classes to the queue from re-entrancy, e.g. loading modules.
	TArray<UClass*> Queue;
};

FSubsystemCollectionBase::FSubsystemCollectionBase()
	: Outer(nullptr)
{
}

FSubsystemCollectionBase::FSubsystemCollectionBase(UClass* InBaseType)
	: BaseType(InBaseType)
	, Outer(nullptr)
{
	check(BaseType);
}

USubsystem* FSubsystemCollectionBase::GetSubsystemInternal(UClass* SubsystemClass) const
{
	// It does not make sense to get a subsystem by null class.
	if (!ensure(SubsystemClass))
	{
		return nullptr;
	}
	
#if WITH_EDITOR && UE_BUILD_SHIPPING
	TStringBuilder<200> DebugSubsystemClassName;
	if (SubsystemClass && IsEngineExitRequested())
	{
		SubsystemClass->GetFName().AppendString(DebugSubsystemClassName);
	}
#endif

	USubsystem* SystemPtr = SubsystemMap.FindRef(SubsystemClass);

	if (SystemPtr)
	{
		return SystemPtr;
	}
	else if (const FSubsystemCollectionBase::FSubsystemArray* SystemPtrs = FindAndPopulateSubsystemArray(SubsystemClass))
	{
		// There should be an invariant that this array is not empty, but re-entrancy currently causes that in some scenarios
		if (SystemPtrs->Subsystems.Num() > 0)
		{
			return SystemPtrs->Subsystems[0];
		}
	}

	if (Initialization != nullptr)
	{
		// When attempting to access one subsystem from the initialization of another, check the set of subsystems we intend to create
		// for a subsystem of the exact class or of a derived type
		if (UClass* Class = Initialization->ClassMap.FindRef(SubsystemClass))
		{
			// Cast away const explicitly to initialize a subsystem re-entrantly...
			FSubsystemCollectionBase* MutableThis = const_cast<FSubsystemCollectionBase*>(this);
			return MutableThis->AddAndInitializeValidatedSubsystem(Class);
		}
		else 
		{
			ensureMsgf(false, TEXT("Failed a call to GetSubsystem from within Initialize of another subsystem as the requested type %s was not present in the set of classes we expected to initialize"),
				*SubsystemClass->GetPathName());
		}
	}

	return nullptr;
}

void FSubsystemCollectionBase::ForEachSubsystem(TFunctionRef<void(USubsystem*)> Operation) const
{
	TGuardValue<bool> Guard{bIterating, true};
	for (auto It = SubsystemMap.CreateConstIterator(); It; ++It)
	{
		Operation(It->Value);
	}	
}

FSubsystemCollectionBase::FSubsystemArray& FSubsystemCollectionBase::FindAndPopulateSubsystemArrayInternal(UClass* SubsystemClass) const
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	if (const FSubsystemCollectionBase::FSubsystemArray* Array = FindAndPopulateSubsystemArray(SubsystemClass))
	{
		return *const_cast<FSubsystemArray*>(Array);
	}
	static FSubsystemArray Empty;
	return Empty;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

const FSubsystemCollectionBase::FSubsystemArray* FSubsystemCollectionBase::FindAndPopulateSubsystemArray(UClass* SubsystemClass) const
{
	// NOTE: There is no thread safety here for multiple threads trying to access a subsystem by a base class/interface.
	// Ideally there should be.
	const bool bIsInterface = SubsystemClass->IsChildOf<UInterface>();
	if (!SubsystemArrayMap.Contains(SubsystemClass))
	{
		FSubsystemArray NewList;
		for (auto Iter = SubsystemMap.CreateConstIterator(); Iter; ++Iter)
		{
			UClass* KeyClass = Iter.Key();
			if ((!bIsInterface && KeyClass->IsChildOf(SubsystemClass)) || 
				(bIsInterface && KeyClass->ImplementsInterface(SubsystemClass)))
			{
				NewList.Subsystems.Add(Iter.Value());
			}
		}
		if (NewList.Subsystems.Num())
		{
			return SubsystemArrayMap.Add(SubsystemClass, MakeUnique<FSubsystemArray>(MoveTemp(NewList))).Get();
		}
		// If nothing was found, we don't want to store this key in the map as we remove lists when they become empty.
		// We also don't pass the keys in SubsystemArrayMap to GC.
		return nullptr;
	}

	const TUniquePtr<FSubsystemArray>& List = SubsystemArrayMap.FindChecked(SubsystemClass);
	return List.Get();
}

void FSubsystemCollectionBase::ForEachSubsystemOfClass(UClass* SubsystemClass, TFunctionRef<void(USubsystem*)> Operation) const
{
	if (SubsystemClass == nullptr)
	{
		SubsystemClass = USubsystem::StaticClass();
	}

	if (const FSubsystemArray* List = FindAndPopulateSubsystemArray(SubsystemClass))
	{
		TGuardValue<bool> IterationGuard{List->bIsIterating, true};
		for (int32 i=0; i < List->Subsystems.Num(); ++i)
		{
			Operation(List->Subsystems[i]);
		}
	}
}

void FSubsystemCollectionBase::RemoveSubsystemsInPackages(TConstArrayView<UPackage*> Packages)
{
	// Safety check, cannot remove while iterating any subsystem array.
	for (TPair<UClass*, TUniquePtr<FSubsystemArray>>& Pair : SubsystemArrayMap)
	{
		UE_CLOGF(Pair.Value->bIsIterating, LogSubsystemCollection, Fatal,
			"FSubsystemCollectionBase::RemoveSubsystemsInPackages called while iterating subsystems of type %ls",
			*Pair.Key->GetPathName());
	}

	// Collect from SubsystemMap which stores all live subsystems.
	TArray<USubsystem*> SubsystemsToRemove;
	SubsystemsToRemove.Reserve(SubsystemMap.Num());
	for (const TPair<TObjectPtr<UClass>, TObjectPtr<USubsystem>>& Pair : SubsystemMap)
	{
		if (Packages.Contains(Pair.Key->GetPackage()))
		{
			SubsystemsToRemove.Add(Pair.Value);
		}
	}

	// Do the cleaning up.
	for (USubsystem* Subsystem : SubsystemsToRemove)
	{
		UE_LOGF(LogSubsystemCollection, Verbose,
			"FSubsystemCollectionBase::RemoveSubsystemsInPackages: removing %ls", *Subsystem->GetFullName());
		RemoveAndDeinitializeSubsystem(Subsystem);
	}
}

void FSubsystemCollectionBase::Initialize(UObject* NewOuter)
{
	if (Outer != nullptr)
	{
		// already initialized
		return;
	}

	bDeinitComplete = false; // Reset Deinit flag to allow this collection to be re-used
	Outer = NewOuter;
	check(Outer);
	if (ensure(BaseType) && ensureMsgf(SubsystemMap.Num() == 0, TEXT("Currently don't support repopulation of Subsystem Collections.")))
	{
		check(!IsPopulating()); //Populating collections on multiple threads?
		
		//non-thread-safe use of Global lists, must be from GameThread:
		check(IsInGameThread());

		if (GlobalSubsystemCollections.Num() == 0)
		{
			FSubsystemModuleWatcher::InitializeModuleWatcher();
		}

		UE_LOGF(LogSubsystemCollection, Verbose, "Initializing subsystem collection for %ls with type %ls", *GetNameSafe(NewOuter), *GetNameSafe(BaseType));
		
		TArray<UClass*> SubsystemClasses;
		if (BaseType->IsChildOf(UDynamicSubsystem::StaticClass()))
		{
			for (const TPair<FName, TArray<UClass*>>& ModuleClasses : GlobalDynamicSystemModuleMap)
			{
				for (UClass* SubsystemClass : ModuleClasses.Value)
				{
					if (SubsystemClass->IsChildOf(BaseType))
					{
						SubsystemClasses.Add(SubsystemClass);
					}
				}
			}
		}
		else
		{
			GetDerivedClasses(BaseType, SubsystemClasses, true);
		}

		AddAndInitializeSubsystems(SubsystemClasses);

		ModulesUnloadedHandle = FCoreUObjectDelegates::CompiledInUObjectsRemovedDelegate.AddRaw(this, &FSubsystemCollectionBase::RemoveSubsystemsInPackages);

		// Statically track collections
		GlobalSubsystemCollections.Add(this);
	}
}

FSubsystemCollectionBase::~FSubsystemCollectionBase()
{
	// Deinitialize should have been called before reaching GC object destruction phase
	//  fix users that failed to call it!
	// 
	// TEMP disabled check so we can run without errors for now
	// @todo fix the underlying issue and turn this check back on
	//checkf( Outer == nullptr , TEXT("FSubsystemCollectionBase destructor called before Deinitialize!\n") );

	// ensure that it is called even if client didn't
	//	otherwise a deleted pointer is left in GlobalSubsystemCollections
	Deinitialize();
}

bool FSubsystemCollectionBase::IsPopulating() const
{
	return Initialization != nullptr;
}

void FSubsystemCollectionBase::Deinitialize()
{
	//non-thread-safe use of Global lists, must be from GameThread:
	check(IsInGameThread());
	check(!bIterating);

	// already Deinitialize'd :
	if (Outer == nullptr)
	{
		return;
	}

	if (ModulesUnloadedHandle.IsValid())
	{
		FCoreUObjectDelegates::CompiledInUObjectsRemovedDelegate.Remove(ModulesUnloadedHandle);
		ModulesUnloadedHandle.Reset();
	}
	
	// Remove static tracking 
	GlobalSubsystemCollections.Remove(this);
	if (GlobalSubsystemCollections.IsEmpty())
	{
		FSubsystemModuleWatcher::DeinitializeModuleWatcher();
	}
	
	// Check not iterating any lists
	for (TPair<UClass*, TUniquePtr<FSubsystemArray>>& Pair : SubsystemArrayMap)
	{
		UE_CLOGF(Pair.Value->bIsIterating, LogSubsystemCollection, Fatal, "FSubsystemCollectionBase::Deinitialize called while iterating subsystems of type %ls", *Pair.Key->GetPathName());
	}

	// Deinit and clean up existing systems
	for (auto Iter = SubsystemMap.CreateIterator(); Iter; ++Iter)
	{
		UClass* KeyClass = Iter.Key();
		USubsystem* Subsystem = Iter.Value();
		if (IsValid(Subsystem) && Subsystem->GetClass() == KeyClass)
		{
			Subsystem->Deinitialize();
			Subsystem->InternalOwningSubsystem = nullptr;
		}
	}

	SubsystemArrayMap.Empty();
	SubsystemMap.Empty();
	Outer = nullptr;
	bDeinitComplete = true;
}

USubsystem* FSubsystemCollectionBase::InitializeDependency(TSubclassOf<USubsystem> SubsystemClass)
{
	USubsystem* Subsystem = nullptr;
	if (ensureMsgf(SubsystemClass, TEXT("Attempting to add invalid subsystem as dependency."))
		&& ensureMsgf(SubsystemClass->IsChildOf(BaseType), TEXT("ClassType (%s) must be a subclass of BaseType(%s)."), *SubsystemClass->GetPathName(), *BaseType->GetPathName()))
	{
		UE_LOGF(LogSubsystemCollection, VeryVerbose, "Attempting to initialize subsystem dependency (%ls)", *SubsystemClass->GetName());

		if (ensureMsgf(Initialization != nullptr, TEXT("InitializeDependency() should only be called from System USubsystem::Initialization() implementations.")))
		{
			UClass* ConcreteClass = Initialization->ClassMap.FindRef(SubsystemClass);
			if (!ConcreteClass)
			{
				UE_LOGFMT(LogSubsystemCollection, Verbose, "InitializeDependency for subsystem class {RequestedClass} did not find an existing class which the subsytem intended to initialize, trying to initialize with the class given anyway.",
					FTopLevelAssetPath(SubsystemClass));
				ConcreteClass = SubsystemClass;
			}
			else if (ConcreteClass != SubsystemClass)
			{
				UE_LOGFMT(LogSubsystemCollection, Verbose, "InitializeDependency for subsystem class {RequestedClass} redirected this call to create class {ConcreteClass} based on inheritance hierarchy.", 
					FTopLevelAssetPath(SubsystemClass), FTopLevelAssetPath(ConcreteClass));
			}

			Subsystem = AddAndInitializeSubsystem(ConcreteClass); 
		}

		UE_CLOGF(!Subsystem, LogSubsystemCollection, Log, "Failed to initialize subsystem dependency (%ls)", *SubsystemClass->GetName());
	}

	return Subsystem;
}

void FSubsystemCollectionBase::AddReferencedObjects(UObject* Referencer, FReferenceCollector& Collector)
{
	Collector.AddStableReferenceMap(SubsystemMap);
}

void FSubsystemCollectionBase::AddAndInitializeSubsystems(TConstArrayView<UClass*> SubsystemClasses)
{
	FSubsystemCollectionInitialization LocalInitialization;

	for (UClass* Class : SubsystemClasses)
	{
		if (Class->HasAllClassFlags(CLASS_Abstract) || Class->GetAuthoritativeClass() != Class)
		{
			continue;
		}
		if (!Class->GetDefaultObject<USubsystem>()->ShouldCreateSubsystem(Outer))
		{
			UE_LOGFMT(LogSubsystemCollection, Verbose, "Not creating subsystem of class {Class} as it returned false from ShouldCreateSubsystem",
				FTopLevelAssetPath(Class));
			continue;
		}
		LocalInitialization.ClassMap.Add(Class, Class);
		LocalInitialization.Queue.Add(Class);
	}

	// Store a map of classes we intend to initialize for re-entrant calls to GetSubsystem 
	for (UClass* ConcreteClass : SubsystemClasses)
	{
		// Only process classes which were added mapping to themselves above, not more classes which
		// were added during this loop
		if (LocalInitialization.ClassMap.FindRef(ConcreteClass) != ConcreteClass)
		{
			continue;
		}
		for (UClass* Parent = ConcreteClass->GetSuperClass();
			Parent != nullptr && Parent != BaseType && !LocalInitialization.ClassMap.Contains(Parent);
			Parent = Parent->GetSuperClass())
		{
			LocalInitialization.ClassMap.Add(Parent, ConcreteClass);
		}

		for (const FImplementedInterface& Interface : ConcreteClass->Interfaces)
		{
			if (!LocalInitialization.ClassMap.Contains(Interface.Class))
			{
				LocalInitialization.ClassMap.Add(Interface.Class, ConcreteClass);
			}
		}
	}

	// If we're re-entering initialization (e.g. loading a module during subsystem initialization) then add
	// our classes to the existing set/queue and allow the outer loop to do the creation
	if (Initialization)
	{
		for (const TPair<UClass*, UClass*>& Pair : LocalInitialization.ClassMap)
		{
			if (!Initialization->ClassMap.Contains(Pair.Key))
			{
				Initialization->ClassMap.Add(Pair.Key, Pair.Value);	
			}
		}
		Initialization->Queue.Append(LocalInitialization.Queue);
	}
	else 
	{
		// Only mark ourselves as populating now - some subsystems call GetSubsystem in their ShouldCreateSubsystem 
		TGuardValue PopulatingGuard(Initialization, &LocalInitialization);
		for (TPair<UClass*, UClass*> Pair : LocalInitialization.ClassMap)
		{
			// Only initialize the requested classes, other keys are for finding via the inheritence hierarchy 
			// with re-entrant calls to GetSubsystem
			// Skip creation if we created the subsystem re-entrantly
			if (Pair.Key == Pair.Value && !SubsystemMap.Contains(Pair.Key))
			{
				AddAndInitializeValidatedSubsystem(Pair.Key);
			}
		}
	}
}

USubsystem* FSubsystemCollectionBase::AddAndInitializeSubsystem(UClass* SubsystemClass)
{
	FSubsystemCollectionInitialization TempInitialization;
	TGuardValue PopulatingGuard(Initialization, Initialization ? Initialization : &TempInitialization);

	if (!SubsystemMap.Contains(SubsystemClass))
	{
		// Only add instances for non abstract Subsystems
		if (SubsystemClass && !SubsystemClass->HasAllClassFlags(CLASS_Abstract))
		{
			// Catch any attempt to add a subsystem of the wrong type
			checkf(SubsystemClass->IsChildOf(BaseType), TEXT("ClassType (%s) must be a subclass of BaseType(%s)."), *SubsystemClass->GetName(), *BaseType->GetName());

			// Do not create instances of classes that aren't authoritative.
			if (SubsystemClass->GetAuthoritativeClass() != SubsystemClass)
			{	
				return nullptr;
			}

			UE_SCOPED_ENGINE_ACTIVITY(TEXT("Initializing Subsystem %s"), *SubsystemClass->GetName());

			const USubsystem* CDO = SubsystemClass->GetDefaultObject<USubsystem>();
			if (CDO->ShouldCreateSubsystem(Outer))
			{
				return AddAndInitializeValidatedSubsystem(SubsystemClass);
			}

			UE_LOGF(LogSubsystemCollection, VeryVerbose, "Subsystem does not exist, but CDO choose to not create (%ls)", *SubsystemClass->GetName());
		}
		return nullptr;
	}

	UE_LOGF(LogSubsystemCollection, VeryVerbose, "Subsystem already exists (%ls)", *SubsystemClass->GetName());
	return SubsystemMap.FindRef(SubsystemClass);
}

USubsystem* FSubsystemCollectionBase::AddAndInitializeValidatedSubsystem(UClass* SubsystemClass)
{
	ensureMsgf(!bDeinitComplete, TEXT("FSubsystemCollectionBase::AddAndInitializeValidatedSubsystem was called on an already Deinitialized SubsystemCollection. Class: (%s)"), *GetPathNameSafe(SubsystemClass));
	checkf(Initialization, TEXT("AddAndInitializeValidatedSubsystem should only be called from an initialization context"));
	checkf(!SubsystemMap.Contains(SubsystemClass), TEXT("AddAndInitializeValidatedSubsystem called for a class which has already been added"));

	UE_SCOPED_ENGINE_ACTIVITY(TEXT("Initializing Subsystem %s"), *SubsystemClass->GetName());
	UE_LOG_CONTEXT("SubsystemBeingInitialized", FTopLevelAssetPath(SubsystemClass));

	USubsystem* Subsystem = NewObject<USubsystem>(Outer, SubsystemClass);
	SubsystemMap.Add(SubsystemClass,Subsystem);
	Subsystem->InternalOwningSubsystem = this;
	Subsystem->Initialize(*this);
	
	// Add this new subsystem to any existing maps of base classes to lists of subsystems
	// Not calling FatalErrorIfIteratingSubsystems because adding to the end of the array is safe for index-based iteration
	for (TPair<UClass*, TUniquePtr<FSubsystemArray>>& Pair : SubsystemArrayMap)
	{
		const bool bIsInterface = Pair.Key->IsChildOf<UInterface>();
		if ((!bIsInterface && SubsystemClass->IsChildOf(Pair.Key)) || 
			(bIsInterface && SubsystemClass->ImplementsInterface(Pair.Key)))
		{
			Pair.Value->Subsystems.Add(Subsystem);
		}
	}

	OnSubsystemAdded(Subsystem);

	return Subsystem;
}

void FSubsystemCollectionBase::RemoveAndDeinitializeSubsystem(USubsystem* Subsystem)
{
	ensureMsgf(!bDeinitComplete, TEXT("FSubsystemCollectionBase::RemoveAndDeinitializeSubsystem was called on an already Deinitialized SubsystemCollection. Subsystem: (%s)"), *GetPathNameSafe(Subsystem));
	check(Subsystem);
	check(!bIterating);
	USubsystem* SubsystemFound = SubsystemMap.FindAndRemoveChecked(Subsystem->GetClass());
	check(Subsystem == SubsystemFound);

	const UClass* SubsystemClass = Subsystem->GetClass();

	OnPreRemoveSubsystem(Subsystem);

	TArray<UClass*> ClassesToRemove;

	for (TPair<UClass*, TUniquePtr<FSubsystemArray>>& Pair : SubsystemArrayMap)
	{
		const bool bIsInterface = Pair.Key->IsChildOf<UInterface>();
		if ((!bIsInterface && SubsystemClass->IsChildOf(Pair.Key)) || 
			(bIsInterface && SubsystemClass->ImplementsInterface(Pair.Key)))
		{
			UE_CLOGF(Pair.Value->bIsIterating, LogSubsystemCollection, Fatal, "Attempted to deinitialize subsystem %ls while iterating subsystems of type %ls",
				*Subsystem->GetPathName(),
				*Pair.Key->GetPathName());
			Pair.Value->Subsystems.Remove(Subsystem);

			if (Pair.Value->Subsystems.Num() == 0)
			{
				ClassesToRemove.Add(Pair.Key);
			}
		}
	}

	Subsystem->Deinitialize();
	Subsystem->InternalOwningSubsystem = nullptr;

	for (UClass* ClassToRemove : ClassesToRemove)
	{
		SubsystemArrayMap.Remove(ClassToRemove);
	}
}

void FSubsystemCollectionBase::ActivateExternalSubsystem(UClass* SubsystemClass)
{
	AddAllInstances(MakeConstArrayView(&SubsystemClass, 1));
}

void FSubsystemCollectionBase::DeactivateExternalSubsystem(UClass* SubsystemClass)
{
	RemoveAllInstances(MakeConstArrayView(&SubsystemClass, 1));
}

void FSubsystemCollectionBase::AddAllInstances(TConstArrayView<UClass*> SubsystemClasses)
{
	//non-thread-safe use of Global lists, must be from GameThread:
	check(IsInGameThread());

	TArray<UClass*> RelevantClasses;
	for (FSubsystemCollectionBase* SubsystemCollection : GlobalSubsystemCollections)
	{
		RelevantClasses.Reset();
		for (UClass* Class : SubsystemClasses)
		{
			if (Class->IsChildOf(SubsystemCollection->BaseType))
			{
				RelevantClasses.Add(Class);
			}
		}

		SubsystemCollection->AddAndInitializeSubsystems(RelevantClasses);
	}
}

void FSubsystemCollectionBase::RemoveAllInstances(TConstArrayView<UClass*> SubsystemClasses)
{
	TArray<UObject*> SubsystemsToRemove;
	for (UClass* SubsystemClass : SubsystemClasses)
	{
		SubsystemsToRemove.Reset();
		GetObjectsOfClass(SubsystemClass, SubsystemsToRemove);

		for(UObject* SubsystemObj : SubsystemsToRemove)
		{
			USubsystem* Subsystem = CastChecked<USubsystem>(SubsystemObj);

			if (Subsystem->InternalOwningSubsystem)
			{
				Subsystem->InternalOwningSubsystem->RemoveAndDeinitializeSubsystem(Subsystem);
			}
		}
	}
}




/** FSubsystemModuleWatcher Implementations */
void FSubsystemModuleWatcher::OnModulesChanged(FName ModuleThatChanged, EModuleChangeReason ReasonForChange)
{

	switch (ReasonForChange)
	{
	case EModuleChangeReason::ModuleLoaded:
		AddClassesForModule(ModuleThatChanged);
		break;

	case EModuleChangeReason::ModuleUnloaded:
		RemoveClassesForModule(ModuleThatChanged);
		break;
	}
}


void FSubsystemModuleWatcher::InitializeModuleWatcher()
{
	//non-thread-safe use of Global lists, must be from GameThread:
	check(IsInGameThread());

	check(!ModulesChangedHandle.IsValid());

	// Add Loaded Modules
	TArray<UClass*> SubsystemClasses;
	GetDerivedClasses(UDynamicSubsystem::StaticClass(), SubsystemClasses, true);

	for (UClass* SubsystemClass : SubsystemClasses)
	{
		if (!SubsystemClass->HasAllClassFlags(CLASS_Abstract))
		{
			UPackage* const ClassPackage = SubsystemClass->GetOuterUPackage();
			if (ClassPackage)
			{
				const FName ModuleName = FPackageName::GetShortFName(ClassPackage->GetFName());
				if (FModuleManager::Get().IsModuleLoaded(ModuleName))
				{
					TArray<UClass*>& ModuleSubsystemClasses = GlobalDynamicSystemModuleMap.FindOrAdd(ModuleName);
					ModuleSubsystemClasses.Add(SubsystemClass);
				}
			}
		}
	}

	ModulesChangedHandle = FModuleManager::Get().OnModulesChanged().AddStatic(&FSubsystemModuleWatcher::OnModulesChanged);
}

void FSubsystemModuleWatcher::DeinitializeModuleWatcher()
{
	if (ModulesChangedHandle.IsValid())
	{
		FModuleManager::Get().OnModulesChanged().Remove(ModulesChangedHandle);
		ModulesChangedHandle.Reset();
	}
}

void FSubsystemModuleWatcher::AddClassesForModule(const FName& InModuleName)
{
	//non-thread-safe use of Global lists, must be from GameThread:
	check(IsInGameThread());

	check(! GlobalDynamicSystemModuleMap.Contains(InModuleName));

	// Find the class package for this module
	const UPackage* const ClassPackage = FindPackage(nullptr, *(FString("/Script/") + InModuleName.ToString()));
	if (!ClassPackage)
	{
		return;
	}

	TArray<UClass*> SubsystemClasses;
	TArray<UObject*> PackageObjects;
	GetObjectsWithPackage(ClassPackage, PackageObjects, EGetObjectsFlags::None);
	for (UObject* Object : PackageObjects)
	{
		UClass* const CurrentClass = Cast<UClass>(Object);
		if (CurrentClass && !CurrentClass->HasAllClassFlags(CLASS_Abstract) && CurrentClass->IsChildOf(UDynamicSubsystem::StaticClass()))
		{
			SubsystemClasses.Add(CurrentClass);
		}
	}
	FSubsystemCollectionBase::AddAllInstances(SubsystemClasses);

	if (SubsystemClasses.Num() > 0)
	{
		GlobalDynamicSystemModuleMap.Add(InModuleName, MoveTemp(SubsystemClasses));
	}
}

void FSubsystemModuleWatcher::RemoveClassesForModule(const FName& InModuleName)
{
	//non-thread-safe use of Global lists, must be from GameThread:
	check(IsInGameThread());

	TArray<UClass*>* SubsystemClasses = GlobalDynamicSystemModuleMap.Find(InModuleName);
	if (SubsystemClasses)
	{
		for (UClass* SubsystemClass : *SubsystemClasses)
		{
			FSubsystemCollectionBase::RemoveAllInstances(MakeArrayView(&SubsystemClass, 1));
		}
		GlobalDynamicSystemModuleMap.Remove(InModuleName);
	}
}
