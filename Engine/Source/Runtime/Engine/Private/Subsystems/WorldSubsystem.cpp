// Copyright Epic Games, Inc. All Rights Reserved.

#include "Subsystems/WorldSubsystem.h"
#include "Engine/World.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(WorldSubsystem)

UWorld& UWorldSubsystem::GetWorldRef() const
{
	return *CastChecked<UWorld>(GetOuter(), ECastCheckedType::NullChecked);
}

UWorld* UWorldSubsystem::GetWorld() const
{
	return Cast<UWorld>(GetOuter());
}

bool UWorldSubsystem::ShouldCreateSubsystem(UObject* Outer) const
{
	if (!Super::ShouldCreateSubsystem(Outer))
	{
		return false;
	}

	UWorld* World = CastChecked<UWorld>(Outer);
	return DoesSupportWorldType(World->WorldType);
}

bool UWorldSubsystem::DoesSupportWorldType(const EWorldType::Type WorldType) const
{
	return WorldType == EWorldType::Game || WorldType == EWorldType::Editor || WorldType == EWorldType::PIE;
}

void UWorldSubsystem::PostInitialize()
{
	ensureAlwaysMsgf(!bHasCalledPostInitialize, TEXT("PostInitialize has already been called for subsystem %s"), *GetClass()->GetName());
	bHasCalledPostInitialize = true;
}

void UWorldSubsystem::OnWorldBeginPlay(UWorld& InWorld)
{
	ensureAlwaysMsgf(!bHasCalledBeginPlay, TEXT("OnWorldBeginPlay has already been called for subsystem %s"), *GetClass()->GetName());
	bHasCalledBeginPlay = true;
}

void UWorldSubsystem::EnsureHasCalledPostInitialize() const
{
	ensureAlwaysMsgf(bHasCalledPostInitialize, TEXT("PostInitialize has not been called for subsystem %s, check for missing Super::PostInitialize call."), *GetClass()->GetName());
}
void UWorldSubsystem::EnsureHasCalledBeginPlay() const
{
	ensureAlwaysMsgf(bHasCalledBeginPlay, TEXT("OnWorldBeginPlay has not been called for subsystem %s, check for missing Super::OnWorldBeginPlay call."), *GetClass()->GetName());
}

// ----------------------------------------------------------------------------------

UTickableWorldSubsystem::UTickableWorldSubsystem()
	: FTickableGameObject(ETickableTickType::Never)
{

}

UWorld* UTickableWorldSubsystem::GetTickableGameObjectWorld() const
{
	return GetWorld();
}

ETickableTickType UTickableWorldSubsystem::GetTickableTickType() const 
{
	// If this is a template or has not been initialized yet, set to never tick and it will be enabled when it is initialized
	if (IsTemplate() || !bInitialized)
	{
		return ETickableTickType::Never;
	}

	// Otherwise default to conditional
	return ETickableTickType::Conditional;
}

bool UTickableWorldSubsystem::IsAllowedToTick() const
{
	// This function is now deprecated and subclasses should implement IsTickable instead
	// This should never be false because Initialize should always be called before the first tick and Deinitialize cancels the tick
	ensureMsgf(bInitialized, TEXT("Tickable subsystem %s tried to tick when not initialized! Check for missing Super call"), *GetFullName());

	return bInitialized;
}

void UTickableWorldSubsystem::Tick(float DeltaTime)
{
	checkf(IsInitialized(), TEXT("Ticking should have been disabled for an uninitialized subsystem!"));
}

void UTickableWorldSubsystem::Initialize(FSubsystemCollectionBase& Collection)
{
	Super::Initialize(Collection);
	check(!bInitialized);
	bInitialized = true;

	// Refresh the tick type after initialization
	SetTickableTickType(GetTickableTickType());
}

void UTickableWorldSubsystem::Deinitialize()
{
	Super::Deinitialize();
	check(bInitialized);
	bInitialized = false;

	// Always cancel tick as this is about to be destroyed
	SetTickableTickType(ETickableTickType::Never);
}

void UTickableWorldSubsystem::BeginDestroy()
{
	Super::BeginDestroy();

	ensureMsgf(!bInitialized, TEXT("Tickable subsystem %s was destroyed while still initialized! Check for missing Super::Deinitialize call"), *GetFullName());
}
