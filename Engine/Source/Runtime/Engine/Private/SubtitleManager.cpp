// Copyright Epic Games, Inc. All Rights Reserved.

#include "SubtitleManager.h"
#include "Engine/Engine.h"
#include "CanvasItem.h"
#include "CanvasTypes.h"
#include "Audio.h"
#include "Components/AudioComponent.h"
#include "AudioThread.h"
#include "Engine/World.h"
#include "UnrealClient.h"
#include "EngineLogs.h"

DEFINE_LOG_CATEGORY(LogSubtitle);

#define MULTILINE_SPACING_SCALING 1.1f		// Modifier to the spacing between lines in subtitles
#define SUBTITLE_CHAR_HEIGHT 24

/** The default offset of the outline box. */
FIntRect DrawStringOutlineBoxOffset(2, 2, 4, 4);

void FSubtitleManager::KillAllSubtitles()
{
	ActiveSubtitles.Empty();
}

void FSubtitleManager::KillSubtitles(PTRINT SubtitleID)
{
	ActiveSubtitles.Remove(SubtitleID);
}

void FSubtitleManager::QueueSubtitles(const FQueueSubtitleParams& QueueSubtitleParams)
{
	check(IsInAudioThread());

	DECLARE_CYCLE_STAT(TEXT("FGameThreadAudioTask.QueueSubtitles"), STAT_AudioQueueSubtitles, STATGROUP_TaskGraphTasks);

	FAudioThread::RunCommandOnGameThread([QueueSubtitleParams]() {
		UAudioComponent* AudioComponent = UAudioComponent::GetAudioComponentFromID(QueueSubtitleParams.AudioComponentID);

		if (AudioComponent && AudioComponent->OnQueueSubtitles.IsBound())
		{
			// intercept the subtitles if the delegate is set
			AudioComponent->OnQueueSubtitles.ExecuteIfBound(QueueSubtitleParams.Subtitles, QueueSubtitleParams.Duration);
		}
		else if (UWorld* World = QueueSubtitleParams.WorldPtr.Get())
		{
			// otherwise, pass them on to the subtitle manager for display
			// Subtitles are hashed based on the associated sound (wave instance).
			FSubtitleManager::GetSubtitleManager()->QueueSubtitles(
				QueueSubtitleParams.WaveInstance,
				QueueSubtitleParams.SubtitlePriority,
				QueueSubtitleParams.bManualWordWrap,
				QueueSubtitleParams.bSingleLine,
				QueueSubtitleParams.Duration,
				QueueSubtitleParams.Subtitles,
				QueueSubtitleParams.RequestedStartTime,
				World->GetAudioTimeSeconds()
			);
		}
	}, GET_STATID(STAT_AudioQueueSubtitles));
}

void FSubtitleManager::QueueSubtitles(PTRINT SubtitleID, float Priority, bool bManualWordWrap, bool bSingleLine, float SoundDuration, const TArray<FSubtitleCue>& Subtitles, float InStartTime, float InCurrentTime)
{
	check(GEngine);
	check(IsInGameThread());

	if ((Subtitles.Num() == 0) || (Priority == 0.0f))
	{
		return; // nothing to show, or subtitle suppressed
	}

	if (SoundDuration == 0.0f)
	{
		UE_LOGF(LogSubtitle, Warning, "Received subtitle with no sound duration.");
		return;
	}

	if (SubtitleID == 0)
	{	// NOTE: This probably oughtn't happen, but since it does happen and this is correct handling of that case, it's verbose rather than a warning.
		UE_LOGF(LogSubtitle, Verbose, "Received subtitle with SubtitleID of 0, which likely means the sound associated with it is actually done.  Will not add this subtitle.");
		return;
	}

	// skip subtitles that have already been displayed
	TArray<FSubtitleCue> SubtitlesToAdd;
	{
		for (int32 SubtitleIndex = 1; SubtitleIndex < Subtitles.Num(); ++SubtitleIndex)
		{
			if (Subtitles[SubtitleIndex].Time >= InStartTime)
			{
				SubtitlesToAdd.Add(Subtitles[SubtitleIndex - 1]);
			}
		}

		SubtitlesToAdd.Add(Subtitles.Last());
	}

	// resolve time offsets to absolute time
	for (FSubtitleCue& Subtitle : SubtitlesToAdd)
	{
		if (Subtitle.Time > SoundDuration)
		{
			Subtitle.Time = SoundDuration;
			UE_LOGF(LogSubtitle, Warning, "Subtitle has time offset greater than length of sound - clamping");
		}

		Subtitle.Time -= InStartTime;
		Subtitle.Time += InCurrentTime;
	}

	// append new subtitles
	FActiveSubtitle& NewSubtitle = ActiveSubtitles.Add(SubtitleID, FActiveSubtitle(0, Priority, bManualWordWrap, bSingleLine, SubtitlesToAdd));

	// add on a blank at the end to clear
	FSubtitleCue& Temp = NewSubtitle.Subtitles[NewSubtitle.Subtitles.AddDefaulted()];
	{
		Temp.Text = FText::GetEmpty();
		Temp.Time = InCurrentTime + (SoundDuration - InStartTime);
	}
}

void FSubtitleManager::DisplaySubtitle(FCanvas* Canvas, FActiveSubtitle* Subtitle, FIntRect& Parms, const FLinearColor& Color)
{
	if (OnSetSubtitleTextDelegate.IsBound())
	{
		// If we have subtitle displays, they should be rendered directly through those, not via the canvas.
		DisplaySubtitle_ToDisplays(Subtitle);
		return;
	}

	// These should be valid in here
	check(GEngine);
	check(Canvas);
	check(Subtitle != nullptr);

	CurrentSubtitleHeight = 0.0f;

	// This can be NULL when there's an asset mixup (especially with localization)
	if (!GEngine->GetSubtitleFont())
	{
		UE_LOGF(LogSubtitle, Warning, "NULL GEngine->GetSubtitleFont() - subtitles not rendering!");
		return;
	}

	float FontHeight = GEngine->GetSubtitleFont()->GetMaxCharHeight();
	float HeightTest = Canvas->GetRenderTarget()->GetSizeXY().Y;
	int32 SubtitleHeight = FMath::TruncToInt((FontHeight * MULTILINE_SPACING_SCALING));
	FIntRect BackgroundBoxOffset = DrawStringOutlineBoxOffset;

	// Needed to add a drop shadow and doing all 4 sides was the only way to make them look correct.  If this shows up as a framerate hit we'll have to think of a different way of dealing with this.
	if (Subtitle->bSingleLine)
	{
		const FText& SubtitleText = Subtitle->Subtitles[Subtitle->Index].Text;

		if (!SubtitleText.IsEmpty())
		{
			
			// Display lines up from the bottom of the region
			Parms.Max.Y -= SUBTITLE_CHAR_HEIGHT;
			
			FCanvasTextItem TextItem(FVector2D(Parms.Min.X + (Parms.Width() / 2), Parms.Max.Y), SubtitleText , GEngine->GetSubtitleFont(), Color);
			{
				TextItem.Depth = SUBTITLE_SCREEN_DEPTH_FOR_3D;
				TextItem.bOutlined = true;
				TextItem.bCentreX = true;
				TextItem.OutlineColor = FLinearColor::Black;
			}

			Canvas->DrawItem(TextItem);		
			CurrentSubtitleHeight += SubtitleHeight;
		}
	}
	else
	{
		for (int32 Idx = Subtitle->Subtitles.Num() - 1; Idx >= 0; Idx--)
		{
			const FText& SubtitleText = Subtitle->Subtitles[Idx].Text;

			// Display lines up from the bottom of the region
			if (!SubtitleText.IsEmpty())
			{
				FCanvasTextItem TextItem(FVector2D(Parms.Min.X + (Parms.Width() / 2), Parms.Max.Y), SubtitleText, GEngine->GetSubtitleFont(), Color);
				{
					TextItem.Depth = SUBTITLE_SCREEN_DEPTH_FOR_3D;
					TextItem.bOutlined = true;
					TextItem.bCentreX = true;
					TextItem.OutlineColor = FLinearColor::Black;
				}

				Canvas->DrawItem(TextItem);

				Parms.Max.Y -= SubtitleHeight;
				CurrentSubtitleHeight += SubtitleHeight;

				// Don't overlap subsequent boxes...
				BackgroundBoxOffset.Max.Y = BackgroundBoxOffset.Min.Y;
			}
		}
	}
}


FString FSubtitleManager::SubtitleCuesToString(FActiveSubtitle* Subtitle)
{
	FString CuesString;

	if (Subtitle != nullptr)
	{
		for (const FSubtitleCue& Cue : Subtitle->Subtitles)
		{
			if (!Cue.Text.IsEmpty())
			{
				if (!CuesString.IsEmpty())
				{
					CuesString += TEXT('\n');
				}

				CuesString += Cue.Text.ToString();
			}
		}
	}

	return CuesString;
}


void FSubtitleManager::DisplaySubtitle_ToDisplays(FActiveSubtitle* Subtitle)
{
	FString SubtitleString;

	// Always display all movie subtitles
	for (const auto& It : ActiveMovieSubtitles)
	{
		if (It.Value.IsValid())
		{
			if (!SubtitleString.IsEmpty())
			{
				SubtitleString += TEXT('\n');
			}

			SubtitleString += SubtitleCuesToString(It.Value.Get());
		}
	}

	if (Subtitle)
	{
		if (Subtitle->bSingleLine)
		{
			// For a single-line subtitle, just grab the currently active subtitle index
			SubtitleString = Subtitle->Subtitles[Subtitle->Index].Text.ToString();
		}
		else
		{
			// Otherwise, display all the subtitles in the cue
			if (!SubtitleString.IsEmpty())
			{
				SubtitleString += TEXT('\n');
			}

			SubtitleString += SubtitleCuesToString(Subtitle);
		}
	}

	OnSetSubtitleTextDelegate.Broadcast(SubtitleString.IsEmpty() ? FText::GetEmpty() : FText::FromString(SubtitleString));
}


void FSubtitleManager::SplitLinesToSafeZone(FCanvas* Canvas, FIntRect& SubtitleRegion)
{
	FString Concatenated;
	float SoundDuration;
	float StartTime, SecondsPerChar, Cumulative;

	for (TMap<PTRINT, FActiveSubtitle>::TIterator It(ActiveSubtitles); It; ++It)
	{
		FActiveSubtitle& Subtitle = It.Value();
		if (Subtitle.bSplit)
		{
			continue;
		}

		// Concatenate the lines into one (in case the lines were partially manually split)
		Concatenated.Empty(256);

		// Set up the base data
		FSubtitleCue& Initial = Subtitle.Subtitles[0];
		Concatenated = Initial.Text.ToString();
		StartTime = Initial.Time;
		SoundDuration = 0.0f;

		for (int32 SubtitleIndex = 1; SubtitleIndex < Subtitle.Subtitles.Num(); ++SubtitleIndex)
		{
			FSubtitleCue& Subsequent = Subtitle.Subtitles[SubtitleIndex];
			Concatenated += Subsequent.Text.ToString();
			// Last blank entry sets the cutoff time to the duration of the sound
			SoundDuration = Subsequent.Time - StartTime;
		}

		// Adjust concat string to use \n character instead of "/n" or "\n"
		int32 SubIdx = Concatenated.Find(TEXT("/n"));
		while (SubIdx >= 0)
		{
			Concatenated = Concatenated.Left(SubIdx) + ((TCHAR)L'\n') + Concatenated.Right(Concatenated.Len() - (SubIdx + 2));
			SubIdx = Concatenated.Find(TEXT("/n"));
		}

		SubIdx = Concatenated.Find(TEXT("\\n"));
		while (SubIdx >= 0)
		{
			Concatenated = Concatenated.Left(SubIdx) + ((TCHAR)L'\n') + Concatenated.Right(Concatenated.Len() - (SubIdx + 2));
			SubIdx = Concatenated.Find(TEXT("\\n"));
		}

		// Work out a metric for the length of time a line should be displayed
		SecondsPerChar = SoundDuration / Concatenated.Len();

		// Word wrap into lines
		TArray<FWrappedStringElement> Lines;
		FTextSizingParameters RenderParms(0.0f, 0.0f, SubtitleRegion.Width(), 0.0f, GEngine->GetSubtitleFont());
		Canvas->WrapString(RenderParms, 0, *Concatenated, Lines);

		// Set up the times
		Subtitle.Subtitles.Empty();
		Cumulative = 0.0f;

		for (int32 SubtitleIndex = 0; SubtitleIndex < Lines.Num(); ++SubtitleIndex)
		{
			FSubtitleCue& Line = Subtitle.Subtitles[Subtitle.Subtitles.AddDefaulted()];		
			Line.Text = FText::FromString(Lines[SubtitleIndex].Value);
			Line.Time = StartTime + Cumulative;

			Cumulative += SecondsPerChar * Line.Text.ToString().Len();
		}

		// Add in the blank terminating line
		FSubtitleCue& Temp = Subtitle.Subtitles[Subtitle.Subtitles.AddDefaulted()];
		{
			Temp.Text = FText::GetEmpty();
			Temp.Time = StartTime + SoundDuration;
		}

		UE_LOGF(LogAudio, Log, "Splitting subtitle:");

		for (int32 SubtitleIndex = 0; SubtitleIndex < Subtitle.Subtitles.Num() - 1; ++SubtitleIndex)
		{
			FSubtitleCue& Cue = Subtitle.Subtitles[SubtitleIndex];
			FSubtitleCue& NextCue = Subtitle.Subtitles[SubtitleIndex + 1];

			UE_LOGF(LogAudio, Log, " ... '%ls' at %g to %g", *(Cue.Text.ToString()), Cue.Time, NextCue.Time);
		}

		// Mark it as split so it doesn't happen again
		Subtitle.bSplit = true;
	}
}


void FSubtitleManager::TrimRegionToSafeZone(FCanvas* Canvas, FIntRect& InOutSubtitleRegion)
{
	FIntRect SafeZone;
	{
		// display all text within text safe area (80% of the screen width and height)
		SafeZone.Min.X = (10 * Canvas->GetRenderTarget()->GetSizeXY().X) / 100;
		SafeZone.Min.Y = (10 * Canvas->GetRenderTarget()->GetSizeXY().Y) / 100;
		SafeZone.Max.X = Canvas->GetRenderTarget()->GetSizeXY().X - SafeZone.Min.X;
		SafeZone.Max.Y = Canvas->GetRenderTarget()->GetSizeXY().Y - SafeZone.Min.Y;
	}

	// Trim to safe area, but keep everything central
	if ((InOutSubtitleRegion.Min.X < SafeZone.Min.X) || (InOutSubtitleRegion.Max.X > SafeZone.Max.X))
	{
		int32 Delta = FMath::Max(SafeZone.Min.X - InOutSubtitleRegion.Min.X, InOutSubtitleRegion.Max.X - SafeZone.Max.X);
		InOutSubtitleRegion.Min.X += Delta;
		InOutSubtitleRegion.Max.X -= Delta;
	}

	if (InOutSubtitleRegion.Max.Y > SafeZone.Max.Y)
	{
		InOutSubtitleRegion.Max.Y = SafeZone.Max.Y;
	}
}


PTRINT FSubtitleManager::FindHighestPrioritySubtitle(float CurrentTime)
{
	// Tick the available subtitles and find the highest priority one
	float HighestPriority = -1.0f;
	PTRINT HighestPriorityID = 0;

	for (TMap<PTRINT, FActiveSubtitle>::TIterator It(ActiveSubtitles); It; ++It)
	{
		FActiveSubtitle& Subtitle = It.Value();
		
		// Remove when last entry is reached
		if (Subtitle.Index == Subtitle.Subtitles.Num() - 1)
		{
			It.RemoveCurrent();
			continue;
		}

		FSubtitleCue& SubtitleCue = Subtitle.Subtitles[Subtitle.Index];
		
		if (CurrentTime >= Subtitle.Subtitles[Subtitle.Index + 1].Time)
		{
			Subtitle.Index++;
		}

		if (Subtitle.Priority > HighestPriority)
		{
			HighestPriority = Subtitle.Priority;
			HighestPriorityID = It.Key();
		}
	}

	return(HighestPriorityID);
}


void FSubtitleManager::DisplaySubtitles(FCanvas* InCanvas, FIntRect& InSubtitleRegion, float InAudioTimeSeconds)
{
	if (OnSetSubtitleTextDelegate.IsBound())
	{
		// Prioritize using display objects over the canvas

		PTRINT HighestPriorityID = FindHighestPrioritySubtitle(InAudioTimeSeconds);
		FActiveSubtitle* Subtitle = HighestPriorityID ? ActiveSubtitles.Find(HighestPriorityID) : nullptr;
		DisplaySubtitle_ToDisplays(Subtitle);
	}
	else
	{
		check(GEngine);
		check(InCanvas);

		if (GEngine->bSubtitlesForcedOff || !GEngine->bSubtitlesEnabled)
		{
			return; // do nothing if subtitles are disabled.
		}

		if (!GEngine->GetSubtitleFont())
		{
			UE_LOGF(LogSubtitle, Warning, "NULL GEngine->GetSubtitleFont() - subtitles not rendering!");
			return;
		}

		if (InSubtitleRegion.Area() > 0)
		{
			// Work out the safe zones
			TrimRegionToSafeZone(InCanvas, InSubtitleRegion);

			// If the lines have not already been split, split them to the safe zone now
			SplitLinesToSafeZone(InCanvas, InSubtitleRegion);

			// Find the subtitle to display
			PTRINT HighestPriorityID = FindHighestPrioritySubtitle(InAudioTimeSeconds);

			// Display the highest priority subtitle
			if (HighestPriorityID)
			{
				FActiveSubtitle* Subtitle = ActiveSubtitles.Find(HighestPriorityID);
				DisplaySubtitle(InCanvas, Subtitle, InSubtitleRegion, FLinearColor::White);
			}
			else if (ActiveMovieSubtitles.Num() > 0)
			{
				for (const auto& It : ActiveMovieSubtitles)
				{
					if (It.Value.IsValid())
					{
						DisplaySubtitle(InCanvas, It.Value.Get(), InSubtitleRegion, FLinearColor::White);
					}
				}
			} 
			else 
			{
				CurrentSubtitleHeight = 0.0f;
			}
		}
	}
}


FSubtitleManager* FSubtitleManager::GetSubtitleManager()
{
	static FSubtitleManager* SSubtitleManager = new FSubtitleManager();
	return SSubtitleManager;
}

void FSubtitleManager::SetMovieSubtitle(UObject* SubtitleOwner, const TArray<FString>& Subtitles)
{
	if (SubtitleOwner != nullptr)
	{
		TSharedPtr<FActiveSubtitle>& MovieSubtitle = ActiveMovieSubtitles.FindOrAdd(SubtitleOwner);

		MovieSubtitle.Reset();

		if (Subtitles.Num() > 0)
		{
			TArray<FSubtitleCue> Cues;

			for (const FString& Subtitle : Subtitles)
			{
				FSubtitleCue Cue;
				Cue.Text = FText::FromString(Subtitle);

				Cues.Add(Cue);
			}

			const int32 Index = 0;
			const float Priority = 1.0f;
			const bool bSplit = true;
			const bool bSingleLine = false;

			MovieSubtitle = MakeShared<FActiveSubtitle>(Index, Priority, bSplit, bSingleLine, Cues);
		}
	}
}
