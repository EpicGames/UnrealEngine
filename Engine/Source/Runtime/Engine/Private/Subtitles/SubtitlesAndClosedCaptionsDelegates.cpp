// Copyright Epic Games, Inc. All Rights Reserved.

#include "Subtitles/SubtitlesAndClosedCaptionsDelegates.h"
#include "Subtitles/SubtitlesAndClosedCaptionsTypes.h"

TDelegate<void(FQueueSubtitleParameters, const ESubtitleTiming)> FSubtitlesAndClosedCaptionsDelegates::QueueSubtitle;
TDelegate<void(const FName&, const FName&, const float), FDefaultTSDelegateUserPolicy> FSubtitlesAndClosedCaptionsDelegates::QueueSubtitleFromSoundWaveName;
TDelegate<bool(FSubtitleAssetData)> FSubtitlesAndClosedCaptionsDelegates::IsSubtitleActive;
TDelegate<void(FSubtitleAssetData)> FSubtitlesAndClosedCaptionsDelegates::StopSubtitle;
TDelegate<void()> FSubtitlesAndClosedCaptionsDelegates::StopAllSubtitles;