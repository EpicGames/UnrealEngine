// Copyright Epic Games, Inc. All Rights Reserved.

#include "Subtitles/SubtitlesAndClosedCaptionsTypes.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SubtitlesAndClosedCaptionsTypes)

#if WITH_EDITOR
void USubtitleAssetUserData::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	for (FSubtitleAssetData& Subtitle : Subtitles)
	{
		Subtitle.bCanEditDuration = (Subtitle.SubtitleDurationType == ESubtitleDurationType::UseDurationProperty);
	}

	// Calling Super:: afterwards here so that bCanEditDuration is part of the change that gets put in the Transaction snapshot.
	Super::PostEditChangeProperty(PropertyChangedEvent);
}

void USubtitleAssetUserData::PostInitProperties()
{
	// Calling Super:: first: ensures that everything is fully initialized and settled before fiddling with Transient values.
	Super::PostInitProperties();

	for (FSubtitleAssetData& Subtitle : Subtitles)
	{
		Subtitle.bCanEditDuration = (Subtitle.SubtitleDurationType == ESubtitleDurationType::UseDurationProperty);
	}
}

void USubtitleAssetUserData::PostLoad()
{
	// Calling Super:: first: ensures that everything is fully initialized and settled before fiddling with Transient values.
	Super::PostLoad();

	for (FSubtitleAssetData& Subtitle : Subtitles)
	{
		Subtitle.bCanEditDuration = (Subtitle.SubtitleDurationType == ESubtitleDurationType::UseDurationProperty);
	}
}
#endif // WITH_EDITOR