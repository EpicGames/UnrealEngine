// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	ScalabilityOptions.cpp: Unreal engine HW compat scalability system.
=============================================================================*/

#include "SystemSettings.h"
#include "HAL/IConsoleManager.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/ConfigUtilities.h"
#include "Engine/Texture2D.h"

DEFINE_LOG_CATEGORY_STATIC(LogSystemSettings, Log, All);

static TAutoConsoleVariable<int32> CVarUseMaxQualityMode(
	TEXT("r.MaxQualityMode"),
	0,
	TEXT("If set to 1, override certain system settings to highest quality regardless of performance impact"),
	ECVF_RenderThreadSafe);

/*-----------------------------------------------------------------------------
	FSystemSettings
-----------------------------------------------------------------------------*/

/** Global accessor */
FSystemSettings GSystemSettings;

/**
 * Helpers for writing to specific ini sections
 */
static const TCHAR* GIniSectionGame = TEXT("SystemSettings");
static const TCHAR* GIniSectionEditor = TEXT("SystemSettingsEditor");

static inline const TCHAR* GetSectionName(bool bIsEditor)
{
	// if we are cooking, look for an override on the commandline
	FString OverrideSubName;
	if (FParse::Value(FCommandLine::Get(), TEXT("-SystemSettings="), OverrideSubName))
	{
		// append it to SystemSettings, unless it starts with SystemSettings
		static TCHAR ReturnedOverrideName[256] = TEXT("SystemSettings");
		// cache how long the SystemSettings is
		static int32 SystemSettingsStrLen = FCString::Strlen(ReturnedOverrideName);
		if (FCString::Strnicmp(*OverrideSubName, TEXT("SystemSettings"), SystemSettingsStrLen) == 0)
		{
			FCString::Strncpy(ReturnedOverrideName + SystemSettingsStrLen, (*OverrideSubName + SystemSettingsStrLen),
				UE_ARRAY_COUNT(ReturnedOverrideName) - SystemSettingsStrLen);
		}
		else
		{
			FCString::Strncpy(ReturnedOverrideName + SystemSettingsStrLen, *OverrideSubName,
				UE_ARRAY_COUNT(ReturnedOverrideName) - SystemSettingsStrLen);
		}
		return ReturnedOverrideName;
	}

	// return the proper section for using editor or not
	return bIsEditor ? GIniSectionEditor : GIniSectionGame;
}

/**
 * ctor
 */
FSystemSettingsData::FSystemSettingsData()
{
}

// implements a FKeyValueSink
class FSetCVarFromIni
{
public:
	const FString& IniFilename;

	FSetCVarFromIni(const FString& InIniFilename)
		: IniFilename(InIniFilename)
	{}

	void OnEntry(const TCHAR *Key, const TCHAR* Value)
	{
		UE::ConfigUtilities::OnSetCVarFromIniEntry(*IniFilename, Key, Value, ECVF_SetBySystemSettingsIni);
	}
};

/** Initializes and instance with the values from the given IniSection in the engine ini */
void FSystemSettingsData::LoadFromIni( const TCHAR* IniSection, const FString& IniFilename, bool bAllowMissingValues, bool* FoundValues )
{
	// first, look for a parent section to base off of
	FString BasedOnSection;
	if (GConfig->GetString(IniSection, TEXT("BasedOn"), BasedOnSection, IniFilename))
	{
		// recurse with the BasedOn section if it existed, always allowing for missing values
		LoadFromIni(*BasedOnSection, IniFilename, true, FoundValues);
	}

	// Read all console variables from .ini
	{
		FSetCVarFromIni Delegate(IniFilename);
		FKeyValueSink Visitor;

		Visitor.BindRaw(&Delegate, &FSetCVarFromIni::OnEntry);

		GConfig->ForEachEntry(Visitor, IniSection, IniFilename);

		IConsoleManager::Get().CallAllConsoleVariableSinks();
	}
}

/**
 * Constructor, initializing all member variables.
 */
FSystemSettings::FSystemSettings() :
	bIsEditor( false ),
	Force0Mask(ESFIM_All0),
	Force1Mask(ESFIM_All0)
{
	// there should only ever be one of these
	static int32 InstanceCount = 0;
	++InstanceCount;
	check(InstanceCount == 1);
}

void FSystemSettings::RegisterShowFlagConsoleVariables()
{
	struct FIterSink
	{
		FIterSink(FSystemSettings& InThis) : This(InThis)
		{
		}

		bool OnEngineShowFlag(uint32 InIndex, const FString& InName)
		{
			uint32 ShowFlagIndex = FEngineShowFlags::FindIndexByName(*InName);

			// internal error
			check(ShowFlagIndex != -1);

			// Note: showflags cvars should not be used as options because they are considered as cheat cvars which cannot be altered by the user
			IConsoleManager::Get().RegisterConsoleVariableBitRef(*(FString("ShowFlag.") + InName), 
				*InName,
				ShowFlagIndex,
				(uint8*)&This.Force0Mask,
				(uint8*)&This.Force1Mask,
				TEXT("Allows to override a specific showflag (works in editor and game, \"show\" only works in game and UI only in editor)\n")
				TEXT("Useful to run a build many time with the same showflags (when put in consolevariables.ini like \"showflag.abc=0\")\n")
				TEXT(" 0: force the showflag to be OFF\n")
				TEXT(" 1: force the showflag to be ON\n")
				TEXT(" 2: do not override this showflag (default)"),
				ECVF_Cheat);

			return true;
		}

		bool OnCustomShowFlag(uint32 InIndex, const FString& InName)
		{
			// FEngineShowFlags registers its own integer console variables for custom show flags as they are registered by modules loading
			return false;
		}

		FSystemSettings& This;
	};
		
	FIterSink Sink(*this);

	FEngineShowFlags::IterateAllFlags(Sink);
}

/**
 * a few tests to make sure this platform behaves as we expect it
 * if not: ExtractBoolFromBitfield and SetBoolInBitField needs to be modified
 */
static void TestBitFieldFunctions()
{
#if DO_CHECK
	struct FTestMask
	{
		// 48 bits should be enough for testing endianess 
		uint32 B00:1; uint32 B01:1; uint32 B02:1; uint32 B03:1; uint32 B04:1; uint32 B05:1; uint32 B06:1; uint32 B07:1; uint32 B08:1; uint32 B09:1; uint32 B0a:1; uint32 B0b:1; uint32 B0c:1; uint32 B0d:1; uint32 B0e:1; uint32 B0f:1;
		uint32 B10:1; uint32 B11:1; uint32 B12:1; uint32 B13:1; uint32 B14:1; uint32 B15:1; uint32 B16:1; uint32 B17:1; uint32 B18:1; uint32 B19:1; uint32 B1a:1; uint32 B1b:1; uint32 B1c:1; uint32 B1d:1; uint32 B1e:1; uint32 B1f:1;
		uint32 B20:1; uint32 B21:1; uint32 B22:1; uint32 B23:1; uint32 B24:1; uint32 B25:1; uint32 B26:1; uint32 B27:1; uint32 B28:1; uint32 B29:1; uint32 B2a:1; uint32 B2b:1; uint32 B2c:1; uint32 B2d:1; uint32 B2e:1; uint32 B2f:1;
	};

	FTestMask Mask;
	uint8* Mem = (uint8*)&Mask;

	// test cases
	{
		memset(&Mask, 0, sizeof(FTestMask));
		Mask.B00 = 1;
		check(FMath::ExtractBoolFromBitfield(Mem, 0x00) == true);
		FMath::SetBoolInBitField(Mem, 0x00, false);
		check(!Mask.B00);
	}
	{
		memset(&Mask, 0, sizeof(FTestMask));
		Mask.B11 = 1;
		check(FMath::ExtractBoolFromBitfield(Mem, 0x11) == true);
		FMath::SetBoolInBitField(Mem, 0x11, false);
		check(!Mask.B11);
	}
	{
		memset(&Mask, 0, sizeof(FTestMask));
		Mask.B22 = 1;
		check(FMath::ExtractBoolFromBitfield(Mem, 0x22) == true);
		FMath::SetBoolInBitField(Mem, 0x22, false);
		check(!Mask.B11);
	}
#endif
}

/**
 * Initializes system settings and included texture LOD settings.
 *
 * @param bSetupForEditor	Whether to initialize settings for Editor
 */
void FSystemSettings::Initialize( bool bSetupForEditor )
{
	TestBitFieldFunctions();

	RegisterShowFlagConsoleVariables();

	// Load the settings that will be the default for every other compat level, and the editor, and the other split screen levels.
	FSystemSettingsData DefaultSettings;
	DefaultSettings.LoadFromIni(GetSectionName(false), GEngineIni, false);

	bIsEditor = bSetupForEditor;

	(FSystemSettingsData&)(*this) = DefaultSettings;
	LoadFromIni();

	ApplyOverrides();

	IConsoleManager::Get().RegisterConsoleVariableSink_Handle(FConsoleCommandDelegate::CreateRaw(this, &FSystemSettings::CVarSink));

	// initialize a critical texture streaming value used by texture loading, etc
	int32 MinTextureResidentMipCount = 7;
	GConfig->GetInt(TEXT("TextureStreaming"), TEXT("MinTextureResidentMipCount"), MinTextureResidentMipCount, GEngineIni);
	UTexture2D::SetMinTextureResidentMipCount(MinTextureResidentMipCount);
}

void FSystemSettings::CVarSink()
{
	ApplyOverrides();
}

#if UE_ALLOW_EXEC_COMMANDS
bool FSystemSettings::Exec( UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar )
{
	// no longer needed, we have the "Scalability" console command
	return false;
}
#endif // UE_ALLOW_EXEC_COMMANDS

/*-----------------------------------------------------------------------------
	Resolution.
-----------------------------------------------------------------------------*/

/**
 * Overridden function that selects the proper ini section to write to
 */
void FSystemSettings::LoadFromIni()
{
	FSystemSettingsData::LoadFromIni(GetSectionName(bIsEditor));
}

void FSystemSettings::ApplyOverrides()
{
	EConsoleVariableFlags SetBy = ECVF_SetByMask;

	if (FPlatformProperties::SupportsWindowedMode())
	{
		if (CVarUseMaxQualityMode.GetValueOnGameThread() != 0)
		{
			SetBy = (EConsoleVariableFlags)(CVarUseMaxQualityMode.AsVariable()->GetFlags() & ECVF_SetByMask);
		}

		if (FParse::Param(FCommandLine::Get(),TEXT("MAXQUALITYMODE")))
		{
			SetBy = ECVF_SetByCommandline;
		}
	}

	if (SetBy != ECVF_SetByMask)
	{
		// Modify various system settings to get the best quality regardless of performance impact
		{
			static auto CVar = IConsoleManager::Get().FindConsoleVariable(TEXT("r.Shadow.MinResolution"));
			CVar->Set(16, SetBy);
		}

		// Disable shadow fading out over distance
		{
			static auto CVar = IConsoleManager::Get().FindConsoleVariable(TEXT("r.Shadow.FadeResolution"));
			CVar->Set(1, SetBy);
		}

		// Increase minimum preshadow resolution
		{
			static auto CVar = IConsoleManager::Get().FindConsoleVariable(TEXT("r.Shadow.MinPreShadowResolution"));
			CVar->Set(16, SetBy);
		}

		// Disable preshadow fading out over distance
		{
			static auto CVar = IConsoleManager::Get().FindConsoleVariable(TEXT("r.Shadow.PreShadowFadeResolution"));
			CVar->Set(1, SetBy);
		}

		// Increase shadow texel density
		{
			static auto CVar = IConsoleManager::Get().FindConsoleVariable(TEXT("r.Shadow.TexelsPerPixel"));
			CVar->Set(4.0f, SetBy);
		}

		// Don't downsample preshadows
		{
			static auto CVar = IConsoleManager::Get().FindConsoleVariable(TEXT("r.Shadow.PreShadowResolutionFactor"));
			CVar->Set(1.0f, SetBy);
		}
	}
}


ENGINE_API void OnSetCVarFromIniEntry(const TCHAR* IniFile, const TCHAR* Key, const TCHAR* Value, uint32 SetBy, bool bAllowCheating)
{
	UE::ConfigUtilities::OnSetCVarFromIniEntry(IniFile, Key, Value, SetBy, bAllowCheating);
}
