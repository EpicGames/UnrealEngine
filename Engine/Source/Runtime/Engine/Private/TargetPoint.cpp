// Copyright Epic Games, Inc. All Rights Reserved.

#include "Engine/TargetPoint.h"
#include "Async/TaskGraphInterfaces.h"
#include "UObject/ConstructorHelpers.h"
#include "Components/ArrowComponent.h"
#include "Components/BillboardComponent.h"
#include "Engine/Texture2D.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(TargetPoint)

ATargetPoint::ATargetPoint(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	USceneComponent* SceneComponent = CreateDefaultSubobject<USceneComponent>(TEXT("SceneComp"));

	RootComponent = SceneComponent;

#if WITH_EDITORONLY_DATA
	SpriteComponent = CreateEditorOnlyDefaultSubobject<UBillboardComponent>(TEXT("Sprite"));
	ArrowComponent = CreateEditorOnlyDefaultSubobject<UArrowComponent>(TEXT("Arrow"));

	if (!IsRunningCommandlet())
	{
		// Structure to hold one-time initialization
		struct FConstructorStatics
		{
			ConstructorHelpers::FObjectFinderOptional<UTexture2D> TargetIconSpawnObject;
			ConstructorHelpers::FObjectFinderOptional<UTexture2D> TargetIconObject;
			FName ID_TargetPoint;
			FText NAME_TargetPoint;
			FConstructorStatics()
				: TargetIconSpawnObject(TEXT("/Engine/EditorMaterials/TargetIconSpawn"))
				, TargetIconObject(TEXT("/Engine/EditorResources/S_TargetPoint"))
				, ID_TargetPoint(TEXT("TargetPoint"))
				, NAME_TargetPoint(NSLOCTEXT("SpriteCategory", "TargetPoint", "Target Points"))
			{
			}
		};
		static FConstructorStatics ConstructorStatics;

		if (SpriteComponent)
		{
			SpriteComponent->Sprite = ConstructorStatics.TargetIconObject.Get();
			SpriteComponent->SetRelativeScale3D_Direct(FVector(0.5f, 0.5f, 0.5f));
			SpriteComponent->SpriteInfo.Category = ConstructorStatics.ID_TargetPoint;
			SpriteComponent->SpriteInfo.DisplayName = ConstructorStatics.NAME_TargetPoint;
			SpriteComponent->bIsScreenSizeScaled = true;

			SpriteComponent->SetupAttachment(SceneComponent);
		}

		if (ArrowComponent)
		{
			ArrowComponent->ArrowColor = FColor(150, 200, 255);

			ArrowComponent->ArrowSize = 0.5f;
			ArrowComponent->bTreatAsASprite = true;
			ArrowComponent->SpriteInfo.Category = ConstructorStatics.ID_TargetPoint;
			ArrowComponent->SpriteInfo.DisplayName = ConstructorStatics.NAME_TargetPoint;
			ArrowComponent->SetupAttachment(SpriteComponent);
			ArrowComponent->bIsScreenSizeScaled = true;

			// Counteract the scaled down parent so that the arrow is large enough to see.
			if (SpriteComponent)
			{
				ArrowComponent->SetRelativeScale3D((FVector::OneVector / SpriteComponent->GetRelativeScale3D()));
			}
		}
	}
#endif // WITH_EDITORONLY_DATA

	SetHidden(true);
	SetCanBeDamaged(false);
}

#if WITH_EDITORONLY_DATA
/** Returns SpriteComponent subobject **/
UBillboardComponent* ATargetPoint::GetSpriteComponent() const { return SpriteComponent; }
/** Returns ArrowComponent subobject **/
UArrowComponent* ATargetPoint::GetArrowComponent() const { return ArrowComponent; }
#endif

