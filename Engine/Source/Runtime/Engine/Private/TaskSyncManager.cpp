// Copyright Epic Games, Inc. All Rights Reserved.

#include "TaskSyncManager.h"
#include "Async/TaskGraphInterfaces.h"
#include "Engine/Engine.h"
#include "Engine/Level.h"
#include "Engine/World.h"
#include "Misc/ScopeLock.h"

#if WITH_EDITOR
#include "Editor.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(TaskSyncManager)

DEFINE_LOG_CATEGORY_STATIC(LogTaskSync, Log, All);

using namespace UE::Tick;

static bool bCreateTaskSyncManager = false;
static FAutoConsoleVariableRef CVarCreateTaskSyncManager(
	TEXT("tick.CreateTaskSyncManager"),
	bCreateTaskSyncManager,
	TEXT("If true, create the experimental Task Sync Manager (will always be true eventually)"));

#if WITH_EDITOR
void UTaskSyncManagerSettings::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	if (FTaskSyncManager* Manager = FTaskSyncManager::Get())
	{
		Manager->ReloadRegisteredData();
	}
}

bool UTaskSyncManagerSettings::SupportsAutoRegistration() const
{
	return bCreateTaskSyncManager;
}
#endif 

FWorldContextId FWorldContextId::GetContextIdForWorld(const UWorld* World)
{
	if (World)
	{
		// TODO this logic could be moved into GEngine so we can reuse it. Using separate int because of TIndirectArray
		constexpr FInternalId PIEWorldStart = 1;
		constexpr FInternalId TestWorldStart = 256;
		uint32 CurrentWorldIndex = 0;
		
		for (const FWorldContext& Context : GEngine->GetWorldContexts())
		{
			if (World == Context.World())
			{
				if (CurrentWorldIndex == 0)
				{
					ensure(Context.WorldType == EWorldType::Game || Context.WorldType == EWorldType::Editor);
					// The first world should either be a game or editor world
					return FWorldContextId(FWorldContextId::DefaultWorldContextId);
				}
#if WITH_EDITOR
				else if (Context.PIEInstance >= 0)
				{
					check(Context.PIEInstance < TestWorldStart - 1);
					return FWorldContextId(Context.PIEInstance + PIEWorldStart);
				}
#endif
				// Behavior is currently unclear for preview/test worlds, some are ticked separately and some are not with no explicit setting
				else if (Context.WorldType == EWorldType::Game || Context.WorldType == EWorldType::GamePreview || Context.WorldType == EWorldType::EditorPreview)
				{
					// Use the context handle number as this always increases for a new context
					return FWorldContextId(TestWorldStart + Context.ContextHandle.GetNumber());
				}
				else
				{
					return FWorldContextId();
				}

			}

			CurrentWorldIndex++;
		}
	}

	return FWorldContextId();
}

FActiveSyncWorkHandle::~FActiveSyncWorkHandle()
{
	Reset();
}

bool FActiveSyncWorkHandle::IsValid() const
{
	return SyncPoint.IsValid();
}

FTickFunction* FActiveSyncWorkHandle::GetDependencyTickFunction()
{
	if (IsInGameThread())
	{
		FActiveSyncPoint* SyncPointFunction = SyncPoint.Get();
		if (SyncPointFunction && !SyncPointFunction->IsTickGroupWork())
		{
			// Tick group work can't be used for dependencies
			return SyncPointFunction;
		}
	}
	return nullptr;
}

bool FActiveSyncWorkHandle::HasReservedWork() const
{
	return bWorkReserved;
}

bool FActiveSyncWorkHandle::HasRequestedWork() const
{
	return bWorkRequested;
}

bool FActiveSyncWorkHandle::HasWorkQueue() const
{
	return bHasWorkQueue;
}

bool FActiveSyncWorkHandle::AbandonWork()
{
	if (ensure(SyncPoint.IsValid()))
	{
		return SyncPoint->AbandonWork(*this);
	}
	return false;
}

bool FActiveSyncWorkHandle::Reset()
{
	if (SyncPoint.IsValid())
	{
		SyncPoint->ResetWorkHandle(*this);
	}
	return false;
}

bool FActiveSyncWorkHandle::ReserveFutureWork(ESyncWorkRepetition Repeat)
{
	if (ensure(SyncPoint.IsValid()))
	{
		return SyncPoint->ReserveFutureWork(*this, Repeat);
	}
	return false;
}

bool FActiveSyncWorkHandle::RequestWork(FTickFunction* WorkFunction, ESyncWorkRepetition Repeat)
{
	if (ensure(SyncPoint.IsValid()))
	{
		return SyncPoint->RequestWork(*this, WorkFunction, Repeat);
	}
	return false;
}


bool FActiveSyncWorkHandle::QueueWorkFunction(FWorkFunction&& WorkFunction)
{
	if (ensure(SyncPoint.IsValid()))
	{
		return SyncPoint->QueueWorkFunction(*this, MoveTemp(WorkFunction));
	}
	return false;
}

bool FActiveSyncWorkHandle::SendQueuedWork(FGraphEventRef* TaskToExtend)
{
	if (ensure(SyncPoint.IsValid()))
	{
		return SyncPoint->SendQueuedWork(*this, TaskToExtend);
	}
	return false;
}

FWorkQueueTickFunction::FWorkQueueTickFunction()
{
	bCanEverTick = true;
	bDispatchManually = true;
	bStartWithTickEnabled = true;

	bSetAsOpen = true;
	bClearAfterExecute = true;
	bOpenAfterExecute = true;
	WorkExecutionIndex = INDEX_NONE;

	static FName QueueWorkTickFunction("QueueWorkTickFunction");
	DebugName = QueueWorkTickFunction;
}

FWorkQueueTickFunction::~FWorkQueueTickFunction()
{
	check(!IsWorkExecuting());
}

void FWorkQueueTickFunction::SetQueueOpen(bool bShouldBeOpen)
{
	bSetAsOpen = bShouldBeOpen;
}

bool FWorkQueueTickFunction::AddWork(FWorkFunction&& Work)
{
	check(!IsWorkExecuting());
	
	if (IsQueueOpen())
	{
		QueuedWork.Add(MoveTemp(Work));
		return true;
	}
	return false;
}

void FWorkQueueTickFunction::ClearWork()
{
	check(!IsWorkExecuting());
	QueuedWork.Reset();
}

void FWorkQueueTickFunction::ExecuteWork()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(WorkQueueTickFunction_ExecuteWork);
	check(!IsWorkExecuting());

	for (WorkExecutionIndex = 0; WorkExecutionIndex < QueuedWork.Num(); WorkExecutionIndex++)
	{
		QueuedWork[WorkExecutionIndex]();
	}

	WorkExecutionIndex = INDEX_NONE;

	if (bClearAfterExecute)
	{
		ClearWork();
	}

	if (bOpenAfterExecute)
	{
		SetQueueOpen(true);
	}
}

void FWorkQueueTickFunction::SetDebugName(FName InDebugName, const FString& InDetailString)
{
	DebugName = InDebugName;
	DebugDetailString = InDetailString;
}

void FWorkQueueTickFunction::ExecuteTick(float DeltaTime, ELevelTick TickType, ENamedThreads::Type CurrentThread, const FGraphEventRef& MyCompletionGraphEvent)
{
	ExecuteWork();
}

FString FWorkQueueTickFunction::DiagnosticMessage()
{
	return FString::Printf(TEXT("%s:%s"), *DebugName.ToString(), *DebugDetailString);
}

FName FWorkQueueTickFunction::DiagnosticContext(bool bDetailed)
{
	if (bDetailed && !DebugDetailString.IsEmpty())
	{
		return FName(*DiagnosticMessage());
	}

	return DebugName;
}

void FWorkQueueTickFunction::SetClearAfterExecute(bool bShouldClear)
{
	check(!IsWorkExecuting());
	bClearAfterExecute = bShouldClear;
}

void FWorkQueueTickFunction::SetOpenAfterExecute(bool bShouldClear)
{
	check(!IsWorkExecuting());
	bClearAfterExecute = bShouldClear;
}


/** Task to execute a tick function manually outside of it's normal tick group */
class FManualTickFunctionTask
{
	FTickFunction* Target;
	ENamedThreads::Type DesiredThread;
	float DeltaSeconds;
	TEnumAsByte<ELevelTick> TickType;

public:
	FManualTickFunctionTask(FTickFunction* InTarget, ENamedThreads::Type InDesiredThread, float InDeltaSeconds, TEnumAsByte<ELevelTick> InTickType)
		: Target(InTarget)
		, DesiredThread(InDesiredThread)
		, DeltaSeconds(InDeltaSeconds)
		, TickType(InTickType)
	{
	}
	static TStatId GetStatId()
	{
		RETURN_QUICK_DECLARE_CYCLE_STAT(FManualTickFunctionTask, STATGROUP_TaskGraphTasks);
	}
	ENamedThreads::Type GetDesiredThread()
	{
		return DesiredThread;
	}
	static ESubsequentsMode::Type GetSubsequentsMode()
	{
		return ESubsequentsMode::TrackSubsequents;
	}

	void DoTask(ENamedThreads::Type CurrentThread, const FGraphEventRef& MyCompletionGraphEvent)
	{
		if (Target && Target->IsTickFunctionEnabled())
		{
			Target->ExecuteTick(DeltaSeconds, TickType, CurrentThread, MyCompletionGraphEvent);
		}
	}
};

FActiveSyncPoint::FActiveSyncPoint()
{
	bCanEverTick = true;
	bDispatchManually = true;
	bStartWithTickEnabled = true;
	FrameStatus = ESyncPointStatus::Unknown;
}

void FActiveSyncPoint::ResetWorkForFrame()
{
	// Don't lock as this is called from a very specific place before dispatching of tasks
	check(IsInGameThread());

	for (int32 i = 0; i < ActiveWork.Num(); i++)
	{
		FActiveSyncWork& CurrentWork = ActiveWork[i];

		check(!CurrentWork.bCurrentlyExecuting);

		CurrentWork.bAlreadyExecuted = false;
		CurrentWork.bLateWorkRequested = false;

		if (CurrentWork.bRequestEveryFrame)
		{
			CurrentWork.bWorkReserved = false;
			CurrentWork.bWorkRequested = true;
		}
		else if (CurrentWork.bReserveEveryFrame)
		{
			CurrentWork.bWorkReserved = true;
		}

		if (!CurrentWork.bWorkRequested && !CurrentWork.bIsWorkQueueFunction && CurrentWork.TickFunction)
		{
			// This could delete a wrapper tick function
			CurrentWork.SetTickFunction(nullptr, false);
		}
		// Don't reset reserved/requested in case they were set before the frame started
	}
}

bool FActiveSyncPoint::HandleFirstTickGroup()
{
	// Don't lock as this is called from a very specific place before dispatching of tasks
	check(IsInGameThread());

	ensure(FrameStatus == ESyncPointStatus::TaskNotCreated);
	switch (SyncPointDescription.ActivationRules)
	{
	case ESyncPointActivationRules::AlwaysActivate:
		// Not technically true, but will be dispatched soon
		FrameStatus = ESyncPointStatus::Dispatched;
		break;
	case ESyncPointActivationRules::WaitForTrigger:
		FrameStatus = ESyncPointStatus::DispatchWaitingForTrigger;
		break;
	case ESyncPointActivationRules::WaitForAllWork:
	case ESyncPointActivationRules::ActivateForAnyWork:
		FrameStatus = ESyncPointStatus::DispatchWaitingForWork;
		
		return IsReadyToProcessWork();
	}
	return false;
}

bool FActiveSyncPoint::IsReadyToProcessWork() const
{
	if (FrameStatus < ESyncPointStatus::DispatchWaitingForWork || FrameStatus == ESyncPointStatus::ExecutionComplete)
	{
		// Too early or late to process work
		return false;
	}

	if (SyncPointDescription.ActivationRules == ESyncPointActivationRules::ActivateForAnyWork)
	{
		// Dispatch if we have any valid requests
		for (const FActiveSyncWork& CurrentWork : ActiveWork)
		{
			if (CurrentWork.bWorkRequested)
			{
				return true;
			}
		}
		return false;
	}

	bool bHasRequest = false;
	bool bHasReservation = false;

	for (const FActiveSyncWork& CurrentWork : ActiveWork)
	{
		if (CurrentWork.bWorkRequested)
		{
			bHasRequest = true;
		}
		else if (CurrentWork.bWorkReserved)
		{
			bHasReservation = true;
		}
	}

	if (bHasRequest && !bHasReservation)
	{
		// Needs to be dispatched if it has no reservations or will start for any work
		return true;
	}
	return false;
}

bool FActiveSyncPoint::IsTooLateToAddWork(bool bWorkReserved) const
{
	if (FrameStatus == ESyncPointStatus::ExecutionComplete || (FrameStatus >= ESyncPointStatus::Executing && !bWorkReserved))
	{
		// Never too late for tick group work
		return !IsTickGroupWork();
	}
	return false;
}

bool FActiveSyncPoint::GetWorkToExecute(float DeltaTime, ELevelTick TickType, ENamedThreads::Type CurrentThread, const FGraphEventRef& MyCompletionGraphEvent)
{
	// Copy work into executing work and possibly delete active work
	UE::TScopeLock WorkScope(WorkLock);

	if (FrameStatus != ESyncPointStatus::Executing)
	{
		check(FrameStatus >= ESyncPointStatus::Dispatched && FrameStatus != ESyncPointStatus::ExecutionComplete);
		check(ExecutingWork.Num() == 0);
		FrameStatus = ESyncPointStatus::Executing;
	}
	
	const int32 NumActive = ActiveWork.Num();
	int32 NumReserved = 0;
	int32 NumExecuting = 0;
	
	// Presize array to make sure there's room, we don't shrink as it will be the same across frames
	ExecutingWork.SetNumUninitialized(NumActive, EAllowShrinking::No);

	for (int32 ActiveIndex = 0; ActiveIndex < NumActive; ActiveIndex++)
	{
		FActiveSyncWork& CurrentWork = ActiveWork[ActiveIndex];
		if (CurrentWork.bCurrentlyExecuting)
		{
			CurrentWork.bCurrentlyExecuting = false;
			CurrentWork.bAlreadyExecuted = true;
			CurrentWork.bWorkRequested = false;
		}
		else if (CurrentWork.bWorkReserved)
		{
			NumReserved++;
		}
		else if (CurrentWork.bWorkRequested && !CurrentWork.bAlreadyExecuted)
		{
			CurrentWork.bCurrentlyExecuting = true;

			ExecutingWork[NumExecuting].ActiveWorkIndex = ActiveIndex;
			ExecutingWork[NumExecuting].TickFunction = CurrentWork.TickFunction;
			NumExecuting++;
		}
	}

	// Set array to number of things copied, this is faster than adding as we go
	ExecutingWork.SetNumUninitialized(NumExecuting, EAllowShrinking::No);

	if (NumExecuting > 0)
	{
		return true;
	}
	else if (NumReserved > 0)
	{
		FrameStatus = ESyncPointStatus::WaitingForMoreWork;

		check(SyncPointDescription.ActivationRules == ESyncPointActivationRules::ActivateForAnyWork);
		check(MyCompletionGraphEvent.IsValid() && !ReactivationEvent.IsValid());

		// Create our redo event but don't dispatch it yet
		// This might get dispatched immediately after leaving the lock here
		ReactivationEvent = FTaskSyncManager::Get()->CreateManualTickTask(WorldContextId, this, DeltaTime, TickType);

		return false;
	}
	else
	{
		if (IsTickGroupWork())
		{
			// Tick group work can restart at any time
			FrameStatus = ESyncPointStatus::WaitingForManualTask;
		}
		else
		{
			FrameStatus = ESyncPointStatus::ExecutionComplete;
		}
		return false;
	}
}

void FActiveSyncPoint::CancelRequestedWork(int32 ActiveWorkIndex, FActiveSyncWork& CurrentWork)
{
	if (CurrentWork.bLateWorkRequested)
	{
		// This turned into a late work request so cancel it
		FTaskSyncManager* Manager = FTaskSyncManager::Get();
		Manager->CancelTemporaryWorkRequest(this, ActiveWorkIndex);
	}

	if (CurrentWork.bWorkRequested && CurrentWork.bCurrentlyExecuting)
	{
		// This is called with a WorkLock when another thread may be in execute tick, so we can only make simple value changes
		for (int32 ExecutingIndex = 0; ExecutingIndex < ExecutingWork.Num(); ExecutingIndex++)
		{
			if (ExecutingWork[ExecutingIndex].ActiveWorkIndex == ActiveWorkIndex)
			{
				ExecutingWork[ExecutingIndex].Invalidate();
			}
		}
	}

	CurrentWork.bWorkRequested = false;
	CurrentWork.bRequestEveryFrame = false;
}

void FActiveSyncPoint::ExecuteFromGameThread(float DeltaTime, ELevelTick TickType)
{
	// TODO turn on a mode to disable some logic?
	FGraphEventRef EmptyRef;
	ExecuteTick(DeltaTime, TickType, ENamedThreads::GameThread, EmptyRef);
}

void FActiveSyncPoint::ExecuteTick(float DeltaTime, ELevelTick TickType, ENamedThreads::Type CurrentThread, const FGraphEventRef& MyCompletionGraphEvent)
{
	// Only lock during acquisition, GetWorkToExecute protects against recursive ticking
	// This is a loop because the worker threads could add new work during execution

	if (SyncPointDescription.EventType == ESyncPointEventType::SimpleEvent)
	{
		FrameStatus = ESyncPointStatus::ExecutionComplete;
		return;
	}

	// TODO add pre/post batch function calls
	TRACE_CPUPROFILER_EVENT_SCOPE(ActiveSyncPoint_ExecuteTick);
	while (GetWorkToExecute(DeltaTime, TickType, CurrentThread, MyCompletionGraphEvent))
	{
		for (int32 i = 0; i < ExecutingWork.Num(); i++)
		{
			// Executing work cannot change size during execution, but TickFunction could be nulled
			FTickFunction* TickFunction = ExecutingWork[i].TickFunction;
			if (TickFunction && TickFunction->IsTickFunctionEnabled())
			{
				TickFunction->ExecuteNestedTick(DeltaTime, TickType, CurrentThread, MyCompletionGraphEvent);
			}
		}
	}
}

FString FActiveSyncPoint::DiagnosticMessage()
{
	return FString::Printf(TEXT("TaskSyncTickFunction: %s"), *SyncPointDescription.RegisteredName.ToString());
}

FName FActiveSyncPoint::DiagnosticContext(bool bDetailed)
{
	return SyncPointDescription.RegisteredName;
}

bool FActiveSyncPoint::DispatchManually()
{
	UE::TScopeLock WorkScope(WorkLock);

	if (CanDispatchManually())
	{
		ensure(FrameStatus == ESyncPointStatus::DispatchWaitingForTrigger || FrameStatus == ESyncPointStatus::DispatchWaitingForWork);
		// Update status
		FrameStatus = ESyncPointStatus::Dispatched;
		return FTickFunction::DispatchManually();
	}
	return false;
}

FActiveSyncWorkHandle FActiveSyncPoint::RegisterWorkHandle()
{
	UE::TScopeLock WorkScope(WorkLock);

	uint32 WorkIndex = AllocateActiveWork();
	FActiveSyncWork& CurrentWork = ActiveWork[WorkIndex];
	check(!CurrentWork.bHasActiveHandle);

	CurrentWork.bHasActiveHandle = true;

	FActiveSyncWorkHandle ReturnHandle;
	ReturnHandle.SyncPoint = AsShared();
	ReturnHandle.WorkIndex = WorkIndex;
	ReturnHandle.bWorkReserved = false;
	ReturnHandle.bWorkRequested = false;
	ReturnHandle.bHasWorkQueue = false;

	return MoveTemp(ReturnHandle);
}

bool FActiveSyncPoint::ReserveFutureWork(FActiveSyncWorkHandle& Handle, ESyncWorkRepetition Repeat)
{
	UE::TScopeLock WorkScope(WorkLock);
	int32 WorkIndex = Handle.GetIndex();

	if (ensure(Handle.IsValid() && ActiveWork.IsValidIndex(WorkIndex)))
	{
		FActiveSyncWork& CurrentWork = ActiveWork[WorkIndex];

		if (ensure(CurrentWork.bHasActiveHandle))
		{
			if (CurrentWork.bIsWorkQueueFunction)
			{
				return false;
			}

			CurrentWork.bReserveEveryFrame = (Repeat == ESyncWorkRepetition::EveryFrame);

			if (Repeat == ESyncWorkRepetition::Never)
			{
				Handle.bWorkReserved = false;

				CurrentWork.bWorkReserved = false;
			}
			else
			{
				Handle.bWorkReserved = true;
				// TODO what do we do if you reserve after requesting this frame?
				CurrentWork.bWorkReserved = true;
			}

			if (IsReadyToProcessWork())
			{
				DispatchWorkTask();
			}

			return true;
		}
	}
	return false;
}

bool FActiveSyncPoint::RequestWork(FActiveSyncWorkHandle& Handle, FTickFunction* WorkFunction, ESyncWorkRepetition Repeat)
{
	UE::TScopeLock WorkScope(WorkLock);
	int32 WorkIndex = Handle.GetIndex();

	if (ensure(Handle.IsValid() && ActiveWork.IsValidIndex(WorkIndex)))
	{
		FActiveSyncWork& CurrentWork = ActiveWork[WorkIndex];

		if (ensure(CurrentWork.bHasActiveHandle))
		{
			if (CurrentWork.bIsWorkQueueFunction)
			{
				return false;
			}

			if (Repeat == ESyncWorkRepetition::Never)
			{
				Handle.bWorkRequested = false;

				CancelRequestedWork(WorkIndex, CurrentWork);
			}
			else 
			{
				Handle.bWorkRequested = true;

				// If the task is completely done or in the middle of processing and we didn't reserve work
				if (IsTooLateToAddWork(CurrentWork.bWorkReserved))
				{
					FTaskSyncManager* Manager = FTaskSyncManager::Get();
					
					// This may schedule it as part of tick group work
					CurrentWork.bLateWorkRequested = Manager->HandleLateWorkRequest(this, WorkIndex, WorkFunction);
				}

				CurrentWork.bWorkReserved = false;
				CurrentWork.bWorkRequested = !CurrentWork.bLateWorkRequested;

				CurrentWork.bRequestEveryFrame = (Repeat == ESyncWorkRepetition::EveryFrame);
				if (CurrentWork.bRequestEveryFrame || CurrentWork.bWorkRequested)
				{
					// Set function ptr for when it executes which may be next frame
					CurrentWork.SetTickFunction(WorkFunction, false);
				}
			}

			if (IsReadyToProcessWork())
			{
				DispatchWorkTask();
			}
			return true;
		}
	}
	return false;
}

bool FActiveSyncPoint::QueueWorkFunction(FActiveSyncWorkHandle& Handle, FWorkFunction&& WorkFunction)
{
	UE::TScopeLock WorkScope(WorkLock);
	int32 WorkIndex = Handle.GetIndex();

	if (ensure(Handle.IsValid() && ActiveWork.IsValidIndex(WorkIndex)))
	{
		FActiveSyncWork& CurrentWork = ActiveWork[WorkIndex];

		if (ensure(CurrentWork.bHasActiveHandle))
		{
			if (CurrentWork.bWorkRequested || CurrentWork.bRequestEveryFrame || CurrentWork.bReserveEveryFrame)
			{
				// Can't convert a work request into queued work
				return false;
			}

			// Allocate a work queue if needed
			if (!CurrentWork.bIsWorkQueueFunction)
			{
				CurrentWork.SetTickFunction(new FWorkQueueTickFunction(), true);
				CurrentWork.bIsWorkQueueFunction = true;
			}

			Handle.bWorkReserved = true;
			Handle.bHasWorkQueue = true;

			CurrentWork.bWorkReserved = true;
			check(CurrentWork.TickFunction && CurrentWork.bIsWorkQueueFunction);

			FWorkQueueTickFunction* QueueFunction = (FWorkQueueTickFunction*)CurrentWork.TickFunction;

			return QueueFunction->AddWork(MoveTemp(WorkFunction));
		}
	}
	return false;
}

bool FActiveSyncPoint::SendQueuedWork(FActiveSyncWorkHandle& Handle, FGraphEventRef* TaskToExtend)
{
	UE::TScopeLock WorkScope(WorkLock);
	int32 WorkIndex = Handle.GetIndex();

	if (ensure(Handle.IsValid() && ActiveWork.IsValidIndex(WorkIndex)))
	{
		FActiveSyncWork& CurrentWork = ActiveWork[WorkIndex];

		if (ensure(CurrentWork.bHasActiveHandle))
		{
			if (!CurrentWork.bIsWorkQueueFunction || CurrentWork.bWorkRequested)
			{
				// Can't convert a work request into queued work
				return false;
			}
			
			check(CurrentWork.TickFunction && CurrentWork.bIsWorkQueueFunction);
			FWorkQueueTickFunction* QueueFunction = (FWorkQueueTickFunction*)CurrentWork.TickFunction;

			if (QueueFunction->HasQueuedWork())
			{
				// Actually dispatch
				Handle.bWorkRequested = true;
				
				// If the task is completely done or in the middle of processing and we didn't reserve work
				if (IsTooLateToAddWork(CurrentWork.bWorkReserved))
				{
					FTaskSyncManager* Manager = FTaskSyncManager::Get();

					// Can pass through the function as this handle will keep it alive or cancel it
					CurrentWork.bLateWorkRequested = Manager->HandleLateWorkRequest(this, WorkIndex, QueueFunction);
				}

				CurrentWork.bWorkReserved = false;
				CurrentWork.bWorkRequested = !CurrentWork.bLateWorkRequested;

				if (IsReadyToProcessWork())
				{
					DispatchWorkTask(TaskToExtend);
				}
			}
			else
			{
				CurrentWork.bWorkReserved = false;
				CurrentWork.bWorkRequested = false;

				Handle.bWorkRequested = false;
				Handle.bWorkReserved = false;
			}

			return true;
		}
	}
	return false;
}

bool FActiveSyncPoint::AbandonWork(FActiveSyncWorkHandle& Handle)
{
	UE::TScopeLock WorkScope(WorkLock);
	int32 WorkIndex = Handle.GetIndex();
	
	if (ensure(Handle.IsValid() && ActiveWork.IsValidIndex(WorkIndex)))
	{
		FActiveSyncWork& CurrentWork = ActiveWork[WorkIndex];

		if (ensure(CurrentWork.bHasActiveHandle))
		{
			if (CurrentWork.bWorkReserved || CurrentWork.bWorkRequested || CurrentWork.bReserveEveryFrame || CurrentWork.bRequestEveryFrame)
			{
				CancelRequestedWork(WorkIndex, CurrentWork);

				CurrentWork.bWorkReserved = false;
				CurrentWork.bReserveEveryFrame = false;

				Handle.bWorkReserved = false;
				Handle.bWorkRequested = false;
			
				if (IsReadyToProcessWork())
				{
					DispatchWorkTask();
				}
			}
			return true;
		}
	}
	return false;
}

bool FActiveSyncPoint::ResetWorkHandle(FActiveSyncWorkHandle& Handle)
{
	UE::TScopeLock WorkScope(WorkLock);
	int32 WorkIndex = Handle.GetIndex();

	if (ensure(Handle.IsValid() && ActiveWork.IsValidIndex(WorkIndex)))
	{
		FActiveSyncWork& CurrentWork = ActiveWork[WorkIndex];

		if (ensure(CurrentWork.bHasActiveHandle))
		{
			bool bShouldCheckWork = CurrentWork.bWorkReserved || CurrentWork.bWorkRequested;

			CancelRequestedWork(WorkIndex, CurrentWork);
			
			CurrentWork.bWorkReserved = false;
			CurrentWork.bReserveEveryFrame = false;
			CurrentWork.bHasActiveHandle = false;

			// ResetWorkForFrame handles clearing the function

			Handle.ResetInternal();

			if (bShouldCheckWork && IsReadyToProcessWork())
			{
				DispatchWorkTask();
			}
			return true;
		}
	}
	return false;
}

uint32 FActiveSyncPoint::AllocateActiveWork()
{
	// Look for deleted slot which will usually exist
	for (int32 i = 0; i < ActiveWork.Num(); i++)
	{
		if (!ActiveWork[i].IsInitialized())
		{
			return i;
		}
	}

	// Add new slot
	check(ActiveWork.Num() < FActiveSyncWorkHandle::MaxWorkIndex - 1);

	return ActiveWork.AddDefaulted();
}

void FActiveSyncPoint::DispatchWorkTask(FGraphEventRef* TaskToExtend)
{
	FGraphEventRef ThisTask;
	if (FrameStatus < ESyncPointStatus::Dispatched)
	{
		ensure(DispatchManually());
	}
	else if (FrameStatus == ESyncPointStatus::WaitingForMoreWork)
	{
		check(ReactivationEvent.IsValid());
		// Dispatch our reactivation event to let the task activate
		ThisTask = MoveTemp(ReactivationEvent);
		ReactivationEvent.SafeRelease();

		FrameStatus = ESyncPointStatus::Dispatched;
		ThisTask->Unlock();
	}
	else if (FrameStatus == ESyncPointStatus::WaitingForManualTask)
	{
		// Spawn a manual task right here
		check(!ReactivationEvent.IsValid());

		ThisTask = FTaskSyncManager::Get()->CreateManualTickTask(WorldContextId, this);
		
		FrameStatus = ESyncPointStatus::Dispatched;
		ThisTask->Unlock();
	}
	else
	{
		// If it gets here the work will get handled by the already dispatched task, but we can extend the passed in task
		check(FrameStatus == ESyncPointStatus::Dispatched || FrameStatus == ESyncPointStatus::Executing);
	}

	if (TaskToExtend)
	{
		if (!ThisTask && IsCompletionHandleValid())
		{
			ThisTask = GetCompletionHandle();
		}

		if (ThisTask)
		{
			(*TaskToExtend)->DontCompleteUntil(MoveTemp(ThisTask));
		}
	}
}

FTaskSyncManager* FTaskSyncManager::Get()
{
	static TUniquePtr<FTaskSyncManager> SingletonInstance;

	if (SingletonInstance.IsValid())
	{
		// 99% case so check ptr first
		return SingletonInstance.Get();
	}

	if (!bCreateTaskSyncManager)
	{
		// Never created
		return nullptr;
	}

	// Make on demand, but only on game thread. This happens early in tick startup so no async work should be happening before it
	check(IsInGameThread());
	SingletonInstance = MakeUnique<FTaskSyncManager>();
	return SingletonInstance.Get();
}

FTaskSyncManager::FTaskSyncManager()
{
	ReloadRegisteredData();

	if (GEngine)
	{
		GEngine->OnWorldContextDestroyed().AddRaw(this, &FTaskSyncManager::OnWorldContextRemove);
	}
	FWorldDelegates::OnPostWorldInitialization.AddRaw(this, &FTaskSyncManager::OnPostWorldInitialization);
#if WITH_EDITOR	
	FEditorDelegates::EndPIE.AddRaw(this, &FTaskSyncManager::EndPIE);
#endif
}

FTaskSyncManager::~FTaskSyncManager()
{
	if (GEngine)
	{
		GEngine->OnWorldContextDestroyed().RemoveAll(this);
	}
	FWorldDelegates::OnPostWorldInitialization.RemoveAll(this);
#if WITH_EDITOR	
	FEditorDelegates::EndPIE.RemoveAll(this);
#endif
}

void FTaskSyncManager::RegisterSyncPointInternal(const FSyncPointDescription& InDescription)
{
	FSyncPointId::FInternalId NewId = ++HighestSyncId;
	FRegisteredSyncPointData& Data = RegisteredDataMap.FindOrAdd(NewId);
	Data.RegisteredId = NewId;
	Data.RegisteredPoint = InDescription;

	RegisteredNameMap.Add(InDescription.RegisteredName, NewId);

	// TODO what if we're already ticking?
}

void FTaskSyncManager::ReloadRegisteredData()
{
	check(IsInGameThread() && !IsTicking());

	UE::TScopeLock Lock(ManagerLock);
	const UTaskSyncManagerSettings* Settings = GetDefault<UTaskSyncManagerSettings>();

	TArray<FSyncPointDescription> OtherSourceDescriptions;

	for (TPair<FSyncPointId::FInternalId, FRegisteredSyncPointData>& DataPair : RegisteredDataMap)
	{
		FSyncPointDescription& Description = DataPair.Value.RegisteredPoint;
		if (!Description.WasLoadedFromSettings())
		{
			OtherSourceDescriptions.Add(Description);
		}
	}

	RegisteredDataMap.Reset();
	RegisteredNameMap.Reset();

	// Clear out all batch data, this is safe since we're not ticking
	for (FBatchData& BatchData : BatchList)
	{
		BatchData.SyncPointData.Reset();
	}

	// Reset the sync point ids
	HighestSyncId = FSyncPointId::InvalidSyncPoint;

	for (const FSyncPointDescription& SyncPoint : Settings->RegisteredSyncPoints)
	{
		if (SyncPoint.EventType == ESyncPointEventType::Invalid || SyncPoint.ActivationRules == ESyncPointActivationRules::Invalid || !SyncPoint.IsValid())
		{
			// Only error outside the editor as this can happen while modifying the project settings
			UE_CLOGF(!GIsEditor, LogTaskSync, Error, "Cannot register invalid sync point %ls from project settings!", *SyncPoint.RegisteredName.ToString());

			continue;
		}
		RegisterSyncPointInternal(SyncPoint);
	}

	for (const FSyncPointDescription& SyncPoint : OtherSourceDescriptions)
	{
		RegisterSyncPointInternal(SyncPoint);
	}
}

bool FTaskSyncManager::GetSyncPointDescription(FName RegisteredName, FSyncPointDescription& OutDescription) const
{
	UE::TScopeLock Lock(ManagerLock);

	if (const FSyncPointId::FInternalId* FoundId = RegisteredNameMap.Find(RegisteredName))
	{
		if (const FRegisteredSyncPointData* FoundData = RegisteredDataMap.Find(*FoundId))
		{
			OutDescription = FoundData->RegisteredPoint;
			return true;
		}
	}
	
	return false;
}

bool FTaskSyncManager::RegisterNewSyncPoint(const FSyncPointDescription& NewDescription)
{
	check(IsInGameThread());

	if (ensure(NewDescription.IsValid() && !NewDescription.WasLoadedFromSettings()))
	{
		UE::TScopeLock Lock(ManagerLock);

		const FSyncPointId::FInternalId* FoundId = RegisteredNameMap.Find(NewDescription.RegisteredName);

		if (!FoundId)
		{
			RegisterSyncPointInternal(NewDescription);
			return true;
		}
		else
		{
			UE_LOGF(LogTaskSync, Error, "Cannot register sync point %ls from source %ls as it already exists!", *NewDescription.RegisteredName.ToString(), *NewDescription.SourceName.ToString());
		}
	}
	else
	{
		UE_LOGF(LogTaskSync, Error, "Cannot register invalid sync point description %ls from source %ls!", *NewDescription.RegisteredName.ToString(), *NewDescription.SourceName.ToString());
	}

	return false;
}

bool FTaskSyncManager::UnregisterSyncPoint(FName RegisteredName, FName SourceName)
{
	check(IsInGameThread());

	if (ensure(!RegisteredName.IsNone() && !SourceName.IsNone()))
	{
		const FSyncPointId::FInternalId* FoundId = RegisteredNameMap.Find(RegisteredName);

		if (!FoundId)
		{
			UE_LOGF(LogTaskSync, Error, "Cannot unregister sync point %ls from source %ls as it does not exist!", *RegisteredName.ToString(), *SourceName.ToString());
			return false;
		}
		else
		{
			if (const FRegisteredSyncPointData* FoundData = RegisteredDataMap.Find(*FoundId))
			{
				if (FoundData->RegisteredPoint.SourceName == SourceName)
				{
					for (FBatchData& BatchData : BatchList)
					{
						DestroySyncPointIfValid(BatchData, *FoundId);
					}

					RegisteredDataMap.Remove(*FoundId);
					RegisteredNameMap.Remove(RegisteredName);
				}
				else
				{
					UE_LOGF(LogTaskSync, Error, "Cannot unregister sync point %ls from source %ls as it was added by source %ls!", *RegisteredName.ToString(), *SourceName.ToString(), *FoundData->RegisteredPoint.SourceName.ToString());

					return false;
				}

				return true;
			}
			else
			{
				UE_LOGF(LogTaskSync, Error, "Cannot unregister sync point %ls from source %ls as it does not exist!", *RegisteredName.ToString(), *SourceName.ToString());
			}
		}
	}
	else
	{
		UE_LOGF(LogTaskSync, Error, "Cannot unregister invalid sync point %ls from source %ls!", *RegisteredName.ToString(), *SourceName.ToString());
	}

	return false;
}

void FTaskSyncManager::StartFrame(const UWorld* InWorld, float InDeltaSeconds, ELevelTick InTickType)
{
	check(InWorld);	
	check(IsInGameThread() && !IsTicking() && CurrentTickGroup == ETickingGroup::TG_MAX);

	UE::TScopeLock Lock(ManagerLock);
	CurrentTickWorld = InWorld;
	CurrentDeltaTime = InDeltaSeconds;
	CurrentTickType = InTickType;
	ULevel* PersistentLevel = InWorld->PersistentLevel.Get();
	FWorldContextId WorldContext(InWorld);

	// For all batches that match the world
	bool bFoundBatch = false;
	for (FBatchData& BatchData : BatchList)
	{
		if (BatchData.WorldContext == WorldContext)
		{
			bFoundBatch = true;
			InitializeBatchForFrame(BatchData, PersistentLevel);
		}
	}

	// Add the default batch if there were 0
	if (!bFoundBatch)
	{
		FBatchContextId BatchContext = CreateNewBatch(WorldContext);
		check(BatchList.IsValidIndex(BatchContext.BatchId));
		InitializeBatchForFrame(BatchList[BatchContext.BatchId], PersistentLevel);
	}
}

void FTaskSyncManager::InitializeBatchForFrame(FBatchData& BatchData, ULevel* PersistentLevel)
{
	for (TPair<FSyncPointId::FInternalId, FRegisteredSyncPointData>& DataPair : RegisteredDataMap)
	{
		FActiveSyncPoint* SyncPoint = GetOrCreateSyncPoint(BatchData, DataPair.Value);
		if (SyncPoint)
		{
			if (!SyncPoint->IsTickFunctionRegistered())
			{
				FSyncPointDescription& Description = DataPair.Value.RegisteredPoint;
				
				// Check our prerequisites now and try to create them if necessary
				// This could resize BatchData.SyncPointData
				for (FName Prerequisite : Description.PrerequisiteSyncGroups)
				{
					FSyncPointId::FInternalId* FoundId = RegisteredNameMap.Find(Prerequisite);

					// TODO ignore or error on missing prereq?
					if (FoundId)
					{
						FRegisteredSyncPointData* FoundData = RegisteredDataMap.Find(*FoundId);
						if (ensure(FoundData))
						{
							FActiveSyncPoint* PrerequisiteSyncPoint = GetOrCreateSyncPoint(BatchData, *FoundData);

							if (PrerequisiteSyncPoint)
							{
								SyncPoint->AddPrerequisite(PersistentLevel, *PrerequisiteSyncPoint);
							}
						}
						
					}
				}

				// This may have been unregistered automatically due to level change
				ensure(SyncPoint->FrameStatus == ESyncPointStatus::TaskNotRegistered || SyncPoint->FrameStatus == ESyncPointStatus::ExecutionComplete);
				SyncPoint->RegisterTickFunction(PersistentLevel);
			}
			SyncPoint->FrameStatus = ESyncPointStatus::TaskNotCreated;
			SyncPoint->ResetWorkForFrame();
		}
	}

	for (int32 Group = 0; Group < BatchData.TickGroupWork.Num(); Group++)
	{
		// Reset the tick group now
		FActiveSyncPoint* TickGroupWork = BatchData.TickGroupWork[Group].Get();
		if (TickGroupWork)
		{
			TickGroupWork->FrameStatus = ESyncPointStatus::TaskNotCreated;
			TickGroupWork->ResetWorkForFrame();
		}
	}
}

FActiveSyncPoint* FTaskSyncManager::GetOrCreateSyncPoint(FBatchData& BatchData, FRegisteredSyncPointData& SyncData)
{
	FSyncPointDescription& Description = SyncData.RegisteredPoint;
	TSharedRef<FActiveSyncPoint>* FoundData = BatchData.SyncPointData.Find(SyncData.RegisteredId);

	if (!FoundData)
	{
		// Check conditions for creation and possibly return 0

		TSharedRef<FActiveSyncPoint> NewTickFunction = MakeShared<FActiveSyncPoint>();
		NewTickFunction->TickGroup = Description.FirstPossibleTickGroup;
		NewTickFunction->EndTickGroup = Description.LastPossibleTickGroup;
		NewTickFunction->FrameStatus = ESyncPointStatus::TaskNotRegistered;

		NewTickFunction->SyncPointDescription = Description;
		NewTickFunction->SyncPointId.BatchContext = BatchData.BatchContext;
		NewTickFunction->SyncPointId.SyncId = SyncData.RegisteredId;
		NewTickFunction->WorldContextId = BatchData.WorldContext;

		switch (Description.EventType)
		{
		case ESyncPointEventType::SimpleEvent:
			// Could possibly be implemented as a base FGraphEvent with some other changes
			NewTickFunction->bRunOnAnyThread = true;
			NewTickFunction->bHighPriority = true;
			break;
		case ESyncPointEventType::GameThreadTask:
			NewTickFunction->bRunOnAnyThread = false;
			NewTickFunction->bHighPriority = false;
			break;
		case ESyncPointEventType::GameThreadTask_HighPriority:
			NewTickFunction->bRunOnAnyThread = false;
			NewTickFunction->bHighPriority = true;
			break;
		case ESyncPointEventType::WorkerThreadTask:
			NewTickFunction->bRunOnAnyThread = true;
			NewTickFunction->bHighPriority = false;
			break;
		case ESyncPointEventType::WorkerThreadTask_HighPriority:
			NewTickFunction->bRunOnAnyThread = true;
			NewTickFunction->bHighPriority = false;
			break;
		default:
			ensureMsgf(0, TEXT("Invalid event type %d on sync point %s!"), EnumToUnderlyingType(Description.EventType), *Description.RegisteredName.ToString());
		}

		switch (Description.ActivationRules)
		{
		case ESyncPointActivationRules::AlwaysActivate:
			NewTickFunction->bDispatchManually = false;
			break;
		case ESyncPointActivationRules::WaitForTrigger:
		case ESyncPointActivationRules::WaitForAllWork:
		case ESyncPointActivationRules::ActivateForAnyWork:
			NewTickFunction->bDispatchManually = true;
			break;
		default:
			ensureMsgf(0, TEXT("Invalid activation rules %d on sync point %s!"), EnumToUnderlyingType(Description.ActivationRules), *Description.RegisteredName.ToString());
		}

		FoundData = &BatchData.SyncPointData.Add(SyncData.RegisteredId, MoveTemp(NewTickFunction));
	}

	return &FoundData->Get();
}

bool FTaskSyncManager::DestroySyncPointIfValid(FBatchData& BatchData, FSyncPointId::FInternalId SyncId)
{
	TSharedRef<FActiveSyncPoint>* FoundData = BatchData.SyncPointData.Find(SyncId);

	if (FoundData)
	{
		// Unregister the tick function which will stop it from ticking if it's still pending
		(*FoundData)->UnRegisterTickFunction();

		if (IsTicking() && FWorldContextId(CurrentTickWorld) == BatchData.WorldContext)
		{
			// If we're currently ticking, we need to keep this alive for a frame
			BatchData.SyncPointsToDelete.Add(FoundData->ToSharedPtr());
		}

		// Remove the shared ptr. If this is the only thing keeping it alive, this will destroy it immediately
		BatchData.SyncPointData.Remove(SyncId);

		return true;
	}
	return false;
}

FActiveSyncPoint* FTaskSyncManager::GetOrCreateTickGroupWork(FBatchData& BatchData, ETickingGroup TickGroup)
{
	check(TickGroup >= 0 && TickGroup < ETickingGroup::TG_MAX);
	if (TickGroup >= BatchData.TickGroupWork.Num())
	{
		BatchData.TickGroupWork.SetNum(TickGroup + 1);
	}

	FActiveSyncPoint* FoundPoint = BatchData.TickGroupWork[TickGroup].Get();
	if (FoundPoint)
	{
		return FoundPoint;
	}

	TSharedRef<FActiveSyncPoint> NewTickFunction = MakeShared<FActiveSyncPoint>();
	NewTickFunction->TickGroup = TickGroup;
	NewTickFunction->EndTickGroup = TickGroup;
	NewTickFunction->WorldContextId = BatchData.WorldContext;
	NewTickFunction->SyncPointDescription.EventType = ESyncPointEventType::GameThreadTask;
	NewTickFunction->SyncPointDescription.ActivationRules = ESyncPointActivationRules::ActivateForAnyWork;

	if (IsTicking() && TickGroup == CurrentTickGroup)
	{
		// Can be used immediately
		NewTickFunction->FrameStatus = ESyncPointStatus::WaitingForManualTask;
	}
	else
	{
		NewTickFunction->FrameStatus = ESyncPointStatus::TaskNotCreated;
	}
	

	check(NewTickFunction->IsTickGroupWork());
	
	BatchData.TickGroupWork[TickGroup] = NewTickFunction;
	return &NewTickFunction.Get();
}

void FTaskSyncManager::StartTickGroup(const UWorld* InWorld, ETickingGroup Group, TArray<FTickFunction*>& TicksToManualDispatch)
{
	check(IsInGameThread() && CurrentTickWorld == InWorld && CurrentTickGroup == ETickingGroup::TG_MAX);

	UE::TScopeLock Lock(ManagerLock);
	FWorldContextId WorldContext(InWorld);

	CurrentTickGroup = Group;

	// For all batches that match the world
	for (FBatchData& BatchData : BatchList)
	{
		if (BatchData.WorldContext == WorldContext)
		{
			for (TPair<FSyncPointId::FInternalId, TSharedRef<FActiveSyncPoint>>& PointPair : BatchData.SyncPointData)
			{
				FActiveSyncPoint& SyncPoint = PointPair.Value.Get();
				if (SyncPoint.GetDescription().FirstPossibleTickGroup == Group && SyncPoint.HandleFirstTickGroup())
				{
					TicksToManualDispatch.Add(&SyncPoint);
				}
			}

			if (BatchData.TickGroupWork.IsValidIndex(Group))
			{
				FActiveSyncPoint* TickGroupWork = BatchData.TickGroupWork[Group].Get();
				if (TickGroupWork)
				{
					// Mark as ready for work
					TickGroupWork->FrameStatus = ESyncPointStatus::WaitingForManualTask;
				}
			}
		}
	}
}

void FTaskSyncManager::EndTickGroup(const UWorld* InWorld, ETickingGroup Group)
{
	check(IsInGameThread() && CurrentTickWorld == InWorld && CurrentTickGroup != ETickingGroup::TG_MAX);

	// For all batches that match the world
	for (FBatchData& BatchData : BatchList)
	{
		if (BatchData.TickGroupWork.IsValidIndex(Group))
		{
			FActiveSyncPoint* TickGroupWork = BatchData.TickGroupWork[Group].Get();
			if (TickGroupWork)
			{
				// Clear out the work queue if anything is left over
				TickGroupWork->ExecuteFromGameThread(CurrentDeltaTime, CurrentTickType);
				TickGroupWork->FrameStatus = ESyncPointStatus::ExecutionComplete;
			}
		}
	}


	CurrentTickGroup = ETickingGroup::TG_MAX;
}

void FTaskSyncManager::EndFrame(const UWorld* InWorld)
{
	check(IsInGameThread() && CurrentTickWorld == InWorld && CurrentTickGroup == ETickingGroup::TG_MAX);
	CurrentTickWorld = nullptr;
	CurrentTickGroup = ETickingGroup::TG_MAX;

	FWorldContextId WorldContext(InWorld);

	// For all batches that match the world
	for (FBatchData& BatchData : BatchList)
	{
		if (BatchData.WorldContext == WorldContext)
		{
			BatchData.TemporaryWorkRequests.Reset();
			BatchData.SyncPointsToDelete.Reset();
		}
	}
}

void FTaskSyncManager::InitializeForWorld(FWorldContextId WorldContext)
{
	check(IsInGameThread() && WorldContext.IsValid());

	// The batch may already exist, if so just initialize it again
	bool bFoundBatch = false;
	for (FBatchData& BatchData : BatchList)
	{
		if (BatchData.WorldContext == WorldContext)
		{
			bFoundBatch = true;
			InitializeBatchForNewWorld(BatchData);
		}
	}

	// Add the default batch if there were 0
	if (!bFoundBatch)
	{
		FBatchContextId BatchContext = CreateNewBatch(WorldContext);
		check(BatchList.IsValidIndex(BatchContext.BatchId));
		InitializeBatchForNewWorld(BatchList[BatchContext.BatchId]);
	}
}

void FTaskSyncManager::InitializeBatchForNewWorld(FBatchData& BatchData)
{
	// Make sure all registered sync points are created.
	for (TPair<FSyncPointId::FInternalId, FRegisteredSyncPointData>& DataPair : RegisteredDataMap)
	{
		GetOrCreateSyncPoint(BatchData, DataPair.Value);
	}
}

void FTaskSyncManager::OnPostWorldInitialization(UWorld* World, const UWorld::InitializationValues IVS)
{
	check(World != nullptr);

	FWorldContextId WorldContext(World);
	if (WorldContext.IsValid())
	{
		InitializeForWorld(World);
	}
}

void FTaskSyncManager::ReleaseWorldContext(FWorldContextId WorldContext)
{
	// We're completely destroying a world context so free the data
	if (ensure(IsInGameThread() && !IsTicking()))
	{
		for (FBatchData& BatchData : BatchList)
		{
			if (BatchData.WorldContext == WorldContext)
			{
				BatchData.Reset();
			}
		}
	}
}

void FTaskSyncManager::OnWorldContextRemove(FWorldContext& InWorldContext)
{
	if (InWorldContext.WorldType != EWorldType::None && InWorldContext.WorldType != EWorldType::Inactive && InWorldContext.World())
	{
		ReleaseWorldContext(FWorldContextId(InWorldContext.World()));
	}
}

#if WITH_EDITOR
void FTaskSyncManager::EndPIE(bool bStartSimulate)
{
	// PIE shutdown doesn't properly call OnWorldContextDestroyed, so iterate the list ourselves
	for (const FWorldContext& Context : GEngine->GetWorldContexts())
	{
		if (Context.WorldType == EWorldType::PIE)
		{
			OnWorldContextRemove(const_cast<FWorldContext&>(Context));
		}
	}
}
#endif

FWorldContextId FTaskSyncManager::GetCurrentWorldContext() const
{
	// Does this make sense to be different per thread? Looking up the world is slow and unnecessary in cooked
	if (ensure(IsInGameThread()))
	{
		// TODO Pie logic
		return FWorldContextId(FWorldContextId::DefaultWorldContextId);
	}
	return FWorldContextId(FWorldContextId::InvalidWorldContextId);
}

FBatchContextId FTaskSyncManager::FindDefaultBatch(FWorldContextId WorldContext) const
{
	UE::TScopeLock ManagerScope(ManagerLock);

	if (WorldContext.IsValid())
	{
		for (const FBatchData& BatchData : BatchList)
		{
			if (BatchData.WorldContext == WorldContext)
			{
				// First batch is the default one
				return BatchData.BatchContext;
			}
		}
	}

	return FBatchContextId();
}

FBatchContextId FTaskSyncManager::CreateNewBatch(FWorldContextId WorldContext)
{
	if (!ensure(IsInGameThread() && WorldContext.IsValid()))
	{
		return FBatchContextId();
	}

	UE::TScopeLock ManagerScope(ManagerLock);

	// First look for an unused slot
	for (int32 InternalId = 0; InternalId < BatchList.Num(); InternalId++)
	{
		FBatchData& ExistingData = BatchList[InternalId];
		if (!ExistingData.BatchContext.IsValid())
		{
			ExistingData.BatchContext.BatchId = InternalId;
			ExistingData.WorldContext = WorldContext;
			return ExistingData.BatchContext;
		}
	}
	
	int32 NewId = BatchList.AddDefaulted();
	BatchList[NewId].BatchContext.BatchId = NewId;
	BatchList[NewId].WorldContext = WorldContext;
	return FBatchContextId(NewId);
}

FSyncPointId FTaskSyncManager::FindSyncPoint(FBatchContextId Batch, FName RegisteredName)
{
	UE::TScopeLock ManagerScope(ManagerLock);
	FSyncPointId ReturnSyncPoint;
	FSyncPointId::FInternalId* FoundId = RegisteredNameMap.Find(RegisteredName);
	if (FoundId)
	{
		ReturnSyncPoint.SyncId = *FoundId;
		ReturnSyncPoint.BatchContext = Batch;
	}

	return ReturnSyncPoint;
}

FSyncPointId FTaskSyncManager::FindSyncPoint(FWorldContextId WorldContext, FName RegisteredName)
{
	UE::TScopeLock ManagerScope(ManagerLock);
	return FindSyncPoint(FindDefaultBatch(WorldContext), RegisteredName);
}

FTickFunction* FTaskSyncManager::GetTickFunctionForSyncPoint(FSyncPointId SyncPoint)
{
	if (!ensure(IsInGameThread()))
	{
		return nullptr;
	}

	UE::TScopeLock ManagerScope(ManagerLock);
	TSharedPtr<FActiveSyncPoint> ActiveData;
	FTaskSyncResult ReturnStatus = FindActiveSyncPoint(SyncPoint, ActiveData);

	if (ReturnStatus)
	{
		// TODO think about this return value more
		return ActiveData.Get();
	}

	return nullptr;
}

FTaskSyncResult FTaskSyncManager::GetTaskGraphEvent(FSyncPointId SyncPoint, FGraphEventRef& OutEventRef)
{
	UE::TScopeLock ManagerScope(ManagerLock);
	TSharedPtr<FActiveSyncPoint> ActiveData;
	FTaskSyncResult ReturnStatus = FindActiveSyncPoint(SyncPoint, ActiveData);
	if (!ReturnStatus)
	{
		return ReturnStatus;
	}

	if (ReturnStatus.WasTaskCreatedForFrame())
	{
		// TODO should we let it return during execution?	
		if (!ReturnStatus.WasActivatedForFrame())
		{
			check(ActiveData.IsValid() && ActiveData->IsCompletionHandleValid());
			OutEventRef = ActiveData->GetCompletionHandle();

			check(!OutEventRef->IsCompleted());

			return ReturnStatus;
		}
	}

	ReturnStatus.OperationResult = ESyncOperationResult::SyncPointStatusIncorrect;
	return ReturnStatus;
}

FTaskSyncResult FTaskSyncManager::TriggerSyncPoint(FSyncPointId SyncPoint)
{
	UE::TScopeLock ManagerScope(ManagerLock);
	TSharedPtr<FActiveSyncPoint> ActiveData;
	FTaskSyncResult ReturnStatus = FindActiveSyncPoint(SyncPoint, ActiveData);
	if (!ReturnStatus)
	{
		return ReturnStatus;
	}

	if (ReturnStatus.WasTaskCreatedForFrame() && !ReturnStatus.WasActivatedForFrame())
	{
		// Check type
		check(ActiveData);
		FSyncPointDescription& Description = ActiveData->SyncPointDescription;
		if (Description.ActivationRules != ESyncPointActivationRules::WaitForTrigger)
		{
			ReturnStatus.OperationResult = ESyncOperationResult::ActivationRulesIncorrect;
			return ReturnStatus;
		}

		ManagerScope.Unlock();

		// TODO Make sure this lock logic is correct, we want to drop the manager lock before it runs the tick on this thread
		UE::TScopeLock WorkScope(ActiveData->WorkLock);

		if (ActiveData->DispatchManually())
		{
			// Return success and the current status
			return FTaskSyncResult(ESyncOperationResult::Success, ActiveData->GetFrameStatus());
		}
	}

	ReturnStatus.OperationResult = ESyncOperationResult::SyncPointStatusIncorrect;
	return ReturnStatus;
}

FTaskSyncResult FTaskSyncManager::TriggerSyncPointAfterEvent(FSyncPointId SyncPoint, FGraphEventRef EventToWaitFor)
{
	UE::TScopeLock ManagerScope(ManagerLock);
	TSharedPtr<FActiveSyncPoint> ActiveData;
	FTaskSyncResult ReturnStatus = FindActiveSyncPoint(SyncPoint, ActiveData);
	if (!ReturnStatus)
	{
		return ReturnStatus;
	}

	check(ActiveData.IsValid());
	if (ReturnStatus.WasTaskCreatedForFrame() && !ReturnStatus.WasActivatedForFrame())
	{
		// Check type
		check(ActiveData);
		FSyncPointDescription& Description = ActiveData->SyncPointDescription;
		if (Description.ActivationRules != ESyncPointActivationRules::WaitForTrigger)
		{
			ReturnStatus.OperationResult = ESyncOperationResult::ActivationRulesIncorrect;
			return ReturnStatus;
		}

		if (ActiveData->CanDispatchManually())
		{
			ManagerScope.Unlock();

			// TODO Make sure this lock logic is correct, we want to drop the manager lock before it runs the tick on this thread
			UE::TScopeLock WorkScope(ActiveData->WorkLock);

			// Can't use DontCompleteUntil on normal tasks that haven't started executing yet
			//ActiveData->GetCompletionHandle()->DontCompleteUntil(EventToWaitFor);

			ActiveData->GetCompletionHandle()->AddPrerequisites(*EventToWaitFor);

			if (ActiveData->DispatchManually())
			{
				// Return success and the current status
				return FTaskSyncResult(ESyncOperationResult::Success, ActiveData->GetFrameStatus());
			}
		}
	}

	ReturnStatus.OperationResult = ESyncOperationResult::SyncPointStatusIncorrect;
	return ReturnStatus;
}

FTaskSyncResult FTaskSyncManager::RegisterWorkHandle(FSyncPointId SyncPoint, FActiveSyncWorkHandle& OutWorkHandle)
{
	UE::TScopeLock ManagerScope(ManagerLock);
	TSharedPtr<FActiveSyncPoint> ActiveData;
	FTaskSyncResult ReturnStatus = FindActiveSyncPoint(SyncPoint, ActiveData);
	if (!ReturnStatus)
	{
		return ReturnStatus;
	}

	check(ActiveData.IsValid());
	FSyncPointDescription& Description = ActiveData->SyncPointDescription;
	if (Description.EventType == ESyncPointEventType::SimpleEvent)
	{
		ReturnStatus.OperationResult = ESyncOperationResult::EventTypeIncorrect;
		return ReturnStatus;
	}

	// TODO what do we do about inactive/disabled tick functions?

	OutWorkHandle = ActiveData->RegisterWorkHandle();

	return ReturnStatus;
}

FTaskSyncResult FTaskSyncManager::RegisterTickGroupWorkHandle(FWorldContextId WorldContext, ETickingGroup TickGroup, FActiveSyncWorkHandle& OutWorkHandle)
{
	UE::TScopeLock ManagerScope(ManagerLock);

	// Find default batch for world
	for (FBatchData& BatchData : BatchList)
	{
		if (BatchData.WorldContext == WorldContext)
		{
			FActiveSyncPoint* TickGroupWork = GetOrCreateTickGroupWork(BatchData, TickGroup);

			if (TickGroupWork)
			{
				OutWorkHandle = TickGroupWork->RegisterWorkHandle();
				return FTaskSyncResult(ESyncOperationResult::Success);
			}
			return FTaskSyncResult(ESyncOperationResult::EventTypeIncorrect);
		}
	}

	return FTaskSyncResult(ESyncOperationResult::WorldNotFound);
}

FTaskSyncResult FTaskSyncManager::FindActiveSyncPoint(FSyncPointId SyncPoint, TSharedPtr<FActiveSyncPoint>& OutData)
{
	// Internal only, does not lock
	if (!SyncPoint.IsValid())
	{
		return ESyncOperationResult::SyncPointInvalid;
	}
	int32 BatchIndex = SyncPoint.BatchContext.BatchId;
	if (BatchList.IsValidIndex(BatchIndex))
	{
		FBatchData& FoundBatch = BatchList[BatchIndex];
		TSharedRef<FActiveSyncPoint>* FoundData = FoundBatch.SyncPointData.Find(SyncPoint.SyncId);
		if (FoundData)
		{
			OutData = *FoundData;
			
			return FTaskSyncResult(ESyncOperationResult::Success, (*FoundData)->GetFrameStatus());
		}
		return ESyncOperationResult::SyncPointNotFound;

	}
	return ESyncOperationResult::BatchNotFound;
}

bool FTaskSyncManager::HandleLateWorkRequest(FActiveSyncPoint* RequestedSyncPoint, int32 RequestingHandle, FTickFunction* TickFunction)
{
	check(RequestedSyncPoint && RequestedSyncPoint->FrameStatus >= ESyncPointStatus::Dispatched);
	ETickingGroup RequestedTickGroup = RequestedSyncPoint->TickGroup;
	FWorldContextId WorldContext = RequestedSyncPoint->GetWorldContextId();
	
	UE::TScopeLock ManagerScope(ManagerLock);

	if (CurrentTickGroup == TG_MAX)
	{
		// Not ticking any more, just queue for next frame
		return false;
	}
	
	if (CurrentTickGroup < TG_MAX && CurrentTickGroup > RequestedTickGroup)
	{
		// Move to the current tick group, which will be processed at the end
		// TODO maybe this should be TG_NewlySpawned in some cases? Need to review the logic for newly-registered tick functions
		RequestedTickGroup = CurrentTickGroup;
	}

	// Find default batch for world
	for (FBatchData& BatchData : BatchList)
	{
		if (BatchData.WorldContext == WorldContext)
		{
			FActiveSyncPoint* TickGroupWork = GetOrCreateTickGroupWork(BatchData, RequestedTickGroup);
			if (ensure(TickGroupWork))
			{
				FTemporaryWorkRequest& WorkRequest = BatchData.TemporaryWorkRequests.Emplace_GetRef(RequestedSyncPoint, RequestingHandle);
				WorkRequest.WorkHandle = TickGroupWork->RegisterWorkHandle();

				ensure(WorkRequest.WorkHandle.RequestWork(TickFunction, ESyncWorkRepetition::Once));
				return true;
			}
		}
	}

	return false;
}

void FTaskSyncManager::CancelTemporaryWorkRequest(FActiveSyncPoint* RequestedSyncPoint, int32 RequestingHandle)
{
	UE::TScopeLock ManagerScope(ManagerLock);

	// Check all batches as the world context may have been lost
	for (FBatchData& BatchData : BatchList)
	{
		for (FTemporaryWorkRequest& WorkRequest : BatchData.TemporaryWorkRequests)
		{
			if (RequestedSyncPoint == WorkRequest.RequestingSyncPoint && RequestingHandle == WorkRequest.RequestingHandle)
			{
				WorkRequest.WorkHandle.Reset();
			}
		}
	}
}

FGraphEventRef FTaskSyncManager::CreateManualTickTask(FWorldContextId WorldContext, FTickFunction* TickFunction, float DeltaTime, ELevelTick TickType)
{
	check(TickFunction);

	ENamedThreads::Type TargetThread = ENamedThreads::SetTaskPriority(TickFunction->bRunOnAnyThread ? ENamedThreads::AnyThread : ENamedThreads::GameThread,
		TickFunction->bHighPriority ? ENamedThreads::HighTaskPriority : ENamedThreads::NormalTaskPriority);

	// Get the world's delta time if needed
	if (DeltaTime < 0.0f && IsTicking())
	{
		DeltaTime = CurrentDeltaTime;
		TickType = CurrentTickType;
	}

	FBaseGraphTask* Task = TGraphTask<FManualTickFunctionTask>::CreateTask(nullptr, ENamedThreads::AnyThread).ConstructAndHold(TickFunction, TargetThread, DeltaTime, TickType);
	return Task->GetCompletionEvent();
}
