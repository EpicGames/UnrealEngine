// Copyright Epic Games, Inc. All Rights Reserved.

#include "IAnalyticsProviderET.h"
#include "Misc/AutomationTest.h"
#include "Misc/App.h"
#include "EngineAnalytics.h"

#if WITH_DEV_AUTOMATION_TESTS

/**
* Artificial Record Event for analytics - Simulates the engine startup simulation.  
*/
IMPLEMENT_SIMPLE_AUTOMATION_TEST(FAnalyticStartUpSimTest, "System.Engine.Analytic.Record Event - Simulate Program Start", EAutomationTestFlags_FeatureMask | EAutomationTestFlags::ClientContext | EAutomationTestFlags::EngineFilter)

bool FAnalyticStartUpSimTest::RunTest(const FString& Parameters)
{
	if (FEngineAnalytics::IsAvailable())
	{
		//Setup the 'Event Attributes'
		TArray<FAnalyticsEventAttribute> EventAttributes;
		EventAttributes.Add(FAnalyticsEventAttribute(TEXT("LoginID"),		FPlatformMisc::GetLoginId()));
		EventAttributes.Add(FAnalyticsEventAttribute(TEXT("OSID"),			FPlatformMisc::GetOperatingSystemId()));
		EventAttributes.Add(FAnalyticsEventAttribute(TEXT("GameName"),		FApp::GetProjectName()));
		EventAttributes.Add(FAnalyticsEventAttribute(TEXT("CommandLine"),	FCommandLine::Get()));
		
		//Record the event with the 'Engine.AutomationTest.Analytics.ProgramStartedEvent' title
		FEngineAnalytics::GetProvider().RecordEvent(TEXT("Engine.AutomationTest.Analytics.ProgramStartedEvent"), EventAttributes);

		//Get the event strings used
		FString LoginIDTest		=	FPlatformMisc::GetLoginId();
		FString OSID			=	FPlatformMisc::GetOperatingSystemId();
		FString GameNameTest	=	FApp::GetProjectName();
		FString CommandLineArgs	=	FCommandLine::Get();

		//Test the strings to verify they have data.
		TestFalse(TEXT("'LoginID' is not expected to be empty!"), LoginIDTest.IsEmpty());
		TestFalse(TEXT("'OperatingSystemID' is not expected to be empty!"), OSID.IsEmpty());
		TestFalse(TEXT("'GameName' is expected."), GameNameTest.IsEmpty());


		//Verify record event is holding the actual data.  This only triggers if the command line argument of 'AnalyticsDisableCaching' was used.
		if ( CommandLineArgs.Contains(TEXT("AnalyticsDisableCaching")) )
		{
			FString FullLoginIDTestEventName = FString::Printf(TEXT("LoginID\":\"%s"), *LoginIDTest);
			FString FullOSIDTestEventName = FString::Printf(TEXT("OSID\":\"%s"), *OSID);

			for ( const FAutomationExecutionEntry& Entry : ExecutionInfo.GetEntries() )
			{
				const FAutomationEvent& Event = Entry.Event;
				if ( Event.Type == EAutomationEventType::Info && Event.Message.Contains(TEXT("Engine.AutomationTest.Analytics.ProgramStartedEvent")) )
				{
					const FString& Message = Event.Message;

					TestTrue(TEXT("Recorded event name is expected to be in the sent event."), Message.Contains(TEXT("Engine.AutomationTest.Analytics.ProgramStartedEvent")));
					TestTrue(TEXT("'LoginID' is expected to be in the sent event."), Message.Contains(*FullLoginIDTestEventName));
					TestTrue(TEXT("'OperatingSystemID' is expected to be in the sent event."), Message.Contains(*FullOSIDTestEventName));
					TestTrue(TEXT("'GameName' is expected to be in the sent event."), Message.Contains(*GameNameTest));
					TestTrue(TEXT("'CommandLine arguments' are expected to be in the sent event."), Message.Contains(TEXT("AnalyticsDisableCaching")));
				}
			}
		}

		return true;
	}

	ExecutionInfo.AddEvent(FAutomationEvent(EAutomationEventType::Info, TEXT("SKIPPED 'FAnalyticStartUpSimTest' test.  EngineAnalytics are not currently available.")));

	return true;	
}


/**
* FAnalyticsEventAttribute Unit Test.
*/
IMPLEMENT_SIMPLE_AUTOMATION_TEST(FAnalyticsEventAttributeUnitTest, "System.Engine.Analytic.EventAttribute Struct Unit Test", EAutomationTestFlags_FeatureMask | EAutomationTestFlags::ClientContext | EAutomationTestFlags::EngineFilter)

bool FAnalyticsEventAttributeUnitTest::RunTest(const FString& Parameters)
{
	if (FEngineAnalytics::IsAvailable())
	{
		FString AttributeName;
		FString AttributeValue;

		AttributeName = "Test of";
		AttributeValue = "FAnalyticsEventAttribute '(const FString InName, const FString& InVaue)'";

		//Setup the 'Event Attributes'
		FAnalyticsEventAttribute EventAttributesFStringTest(AttributeName, AttributeValue);
		TestTrue(TEXT("Expected to take in these type of values '(const FString&, const FString&)'"), (EventAttributesFStringTest.GetName() == TEXT("Test of") && EventAttributesFStringTest.GetValue() == "FAnalyticsEventAttribute '(const FString InName, const FString& InVaue)'"));

		FAnalyticsEventAttribute EventAttributesTCHARTest(AttributeName, TEXT("FAnalyticsEventAttribute '(const FString InName, const TCHAR* InValue)'"));
		TestTrue(TEXT("Expected to take in these type of values '(const FString&, const TCHAR*')"), (EventAttributesTCHARTest.GetName() == TEXT("Test of") && EventAttributesTCHARTest.GetValue() == TEXT("FAnalyticsEventAttribute '(const FString InName, const TCHAR* InValue)'")));

		bool bIsBoolTest = true;
		FAnalyticsEventAttribute EventAttributesBoolTest(AttributeName, bIsBoolTest);
		TestTrue(TEXT("Expected to take in these types of values '(const FString&, bool)'"), (EventAttributesBoolTest.GetName() == TEXT("Test of") && EventAttributesBoolTest.GetValue() == AnalyticsConversionToString(bIsBoolTest)));

		FGuid GuidTest(FGuid::NewGuid());
		FAnalyticsEventAttribute EventAttributesGuidTest(AttributeName, GuidTest);
		TestTrue(TEXT("Expected to take in these type of values '(const FString&, FGuid)'"), (EventAttributesGuidTest.GetName() == TEXT("Test of") && EventAttributesGuidTest.GetValue() == AnalyticsConversionToString(GuidTest)));

		int32 InNumericType = 42;
		FAnalyticsEventAttribute EventAttributesTInValueTest(AttributeName, InNumericType);
		TestTrue(TEXT("Expected to take in a arithmetic type (example int32)"), (EventAttributesTInValueTest.GetName() == TEXT("Test of") && EventAttributesTInValueTest.GetValue() == AnalyticsConversionToString(InNumericType)));

		TArray<int32> InNumericArray;
		InNumericArray.Add(0);
		InNumericArray.Add(1);
		InNumericArray.Add(2);
		FAnalyticsEventAttribute EventAttributesTArray(AttributeName, InNumericArray);
		TestTrue(TEXT("Expected to take in am arithmetic TArray"), (EventAttributesTArray.GetName() == TEXT("Test of") && EventAttributesTArray.GetValue() == TEXT("0,1,2")));

		FString TMapKey1 = "TestKey 1";
		FString TMapKey2 = "TestKey 2";
		FString TMapKey3 = "TestKey 3";
		int32 NumericValue1 = 0;
		int32 NumericValue2 = 1;
		int32 NumericValue3 = 99;
		TMap<FString, int32> InTMap;
		InTMap.Add(TMapKey1, NumericValue1);
		InTMap.Add(TMapKey2, NumericValue2);
		InTMap.Add(TMapKey3, NumericValue3);
		FAnalyticsEventAttribute EventAttributesTMap(AttributeName, InTMap);
		TestTrue(TEXT("Expected to take in a TMap "), (EventAttributesTMap.GetName() == TEXT("Test of") && EventAttributesTMap.GetValue() == TEXT("TestKey 1:0,TestKey 2:1,TestKey 3:99")));

		return true;
	}

	ExecutionInfo.AddEvent(FAutomationEvent(EAutomationEventType::Info, TEXT("SKIPPED 'FAnalyticsEventAttributeUnitTest' test.  EngineAnalytics are not currently available.")));

	return true;
}

#endif //WITH_DEV_AUTOMATION_TESTS
