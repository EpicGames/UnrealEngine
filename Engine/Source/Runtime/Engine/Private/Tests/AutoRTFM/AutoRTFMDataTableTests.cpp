// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tests/TestHarnessAdapter.h"
#include "AutoRTFM.h"
#include "Engine/DataTable.h"

#if WITH_DEV_AUTOMATION_TESTS

TEST_CASE_NAMED(FAutoRTFMDataTableTests, "AutoRTFM.UDataTable", "[ClientContext][ServerContext][CommandletContext][EngineFilter][SupportsAutoRTFM]")
{
	UDataTable* Object = NewObject<UDataTable>();
	Object->RowStruct = NewObject<UScriptStruct>();

	Object->EmptyTable();
}

#endif //WITH_DEV_AUTOMATION_TESTS
