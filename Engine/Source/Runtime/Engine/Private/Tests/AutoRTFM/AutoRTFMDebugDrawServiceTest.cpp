// Copyright Epic Games, Inc. All Rights Reserved.

#include "Debug/DebugDrawService.h"
#include "Engine/Canvas.h"
#include "Misc/AutomationTest.h"
#include "AutoRTFM.h"
#include "UnrealClient.h"
#include "HitProxies.h"

#if WITH_DEV_AUTOMATION_TESTS

#define CHECK_EQ(A, B) \
	UTEST_EQUAL(TEXT(__FILE__ ":" UE_STRINGIZE(__LINE__) ": UTEST_EQUAL_EXPR(" #A ", " #B ")"), A, B)

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FAutoRTFMDebugDrawServiceTest, "AutoRTFM + UDebugDrawService", EAutomationTestFlags::EngineFilter | EAutomationTestFlags::ClientContext | EAutomationTestFlags::ServerContext | EAutomationTestFlags::CommandletContext)

bool FAutoRTFMDebugDrawServiceTest::RunTest(const FString & Parameters)
{
	if (!AutoRTFM::ForTheRuntime::IsAutoRTFMRuntimeEnabled())
	{
		ExecutionInfo.AddEvent(FAutomationEvent(EAutomationEventType::Info, TEXT("SKIPPED 'FAutoRTFMDebugDrawServiceTest' test. AutoRTFM disabled.")));
		return true;
	}

	//ENGINE_API FCanvas(const FGameTime& Time, ERHIFeatureLevel::Type InFeatureLevel, float InDPIScale = 1.0f);

	struct FMyRenderTarget final : public FRenderTarget
	{
		FIntPoint GetSizeXY() const override { return FIntPoint(128); }
	} RenderTarget;

	struct FMyHitProxyConsumer final : public FHitProxyConsumer
	{
		void AddHitProxy(HHitProxy*) override {}
	} HitProxyConsumer;

	FGameTime GameTime;

	FCanvas Canvas(&RenderTarget, &HitProxyConsumer, GameTime, ERHIFeatureLevel::SM5);
	UCanvas* const CanvasObject = NewObject<UCanvas>();
	CanvasObject->Canvas = &Canvas;

	auto Section = [&](const TCHAR* Name, auto&& Test)
	{
		bool Result = Test();
		if (!Result)
		{
			AddError(FString::Printf(TEXT("In section '%s'."), Name), 1);
		}
	};

	Section(TEXT("Transact(Register, Draw, Unregister)"), [&]
	{
		bool bHit = false;
		bool bIsClosed = false;

		AutoRTFM::Transact([&]
		{
			FDebugDrawDelegate Delegate;
			Delegate.BindLambda([&](UCanvas*, APlayerController*)
			{
				bHit = true;
				bIsClosed = AutoRTFM::IsClosed();
			});

			FDelegateHandle Handle = UDebugDrawService::Register(TEXT("Tonemapper"), Delegate);

			FEngineShowFlags EngineShowFlags(EShowFlagInitMode::ESFIM_Game);

			EngineShowFlags.SetTonemapper(true);

			UDebugDrawService::Draw(EngineShowFlags, CanvasObject);

			UDebugDrawService::Unregister(Handle);
		});

		CHECK_EQ(bHit, true);
		CHECK_EQ(bIsClosed, true);

		return true;
	});

	Section(TEXT("Transact(Register, Unregister), Draw"), [&]
	{
		bool bHit = false;

		AutoRTFM::Transact([&]
		{
			FDebugDrawDelegate Delegate;
			Delegate.BindLambda([&](UCanvas*, APlayerController*)
			{
				bHit = true;
			});

			FDelegateHandle Handle = UDebugDrawService::Register(TEXT("Tonemapper"), Delegate);

			UDebugDrawService::Unregister(Handle);
		});

		FEngineShowFlags EngineShowFlags(EShowFlagInitMode::ESFIM_Game);

		EngineShowFlags.SetTonemapper(true);

		UDebugDrawService::Draw(EngineShowFlags, CanvasObject);

		CHECK_EQ(bHit, false);

		return true;
	});

	Section(TEXT("Transact(Register, Abort), Draw"), [&]
	{
		bool bHit = false;

		AutoRTFM::Transact([&]
		{
			FDebugDrawDelegate Delegate;
			Delegate.BindLambda([&](UCanvas*, APlayerController*)
			{
				bHit = true;
			});

			FDelegateHandle Handle = UDebugDrawService::Register(TEXT("Tonemapper"), Delegate);

			AutoRTFM::AbortTransaction();
		});

		FEngineShowFlags EngineShowFlags(EShowFlagInitMode::ESFIM_Game);

		EngineShowFlags.SetTonemapper(true);

		UDebugDrawService::Draw(EngineShowFlags, CanvasObject);

		CHECK_EQ(bHit, false);

		return true;
	});

	return true;
}

#undef CHECK_EQ

#endif //WITH_DEV_AUTOMATION_TESTS
