// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/AutomationTest.h"
#include "AutoRTFM.h"
#include "GameplayTagsManager.h"
#include "GameplayTagContainer.h"

#if WITH_DEV_AUTOMATION_TESTS

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FAutoRTFMGameplayTagTests, "AutoRTFM + FGameplayTag", EAutomationTestFlags::EngineFilter | EAutomationTestFlags::ClientContext | EAutomationTestFlags::ServerContext | EAutomationTestFlags::CommandletContext)

bool FAutoRTFMGameplayTagTests::RunTest(const FString & Parameters)
{
	if (!AutoRTFM::ForTheRuntime::IsAutoRTFMRuntimeEnabled())
	{
		ExecutionInfo.AddEvent(FAutomationEvent(EAutomationEventType::Info, TEXT("SKIPPED 'FAutoRTFMGameplayTagTests' test. AutoRTFM disabled.")));
		return true;
	}

	FGameplayTag Tag;
	FGameplayTag Other;

	bool bResult = true;

	AutoRTFM::ETransactionResult Result = AutoRTFM::Transact([&]
		{
			bResult = Tag.MatchesTag(Other);
		});

	TestTrueExpr(AutoRTFM::ETransactionResult::Committed == Result);
	TestFalseExpr(bResult);

	TArray<FGameplayTag> Parents;

	Result = AutoRTFM::Transact([&]
		{
			UGameplayTagsManager::Get().ExtractParentTags(Tag, Parents);
		});

	TestTrueExpr(AutoRTFM::ETransactionResult::Committed == Result);
	TestTrueExpr(Parents.IsEmpty());

	Result = AutoRTFM::Transact([&]
		{
			bResult = UGameplayTagsManager::Get().RequestGameplayTagParents(Tag).IsEmpty();
		});

	TestTrueExpr(AutoRTFM::ETransactionResult::Committed == Result);
	TestTrueExpr(bResult);
	
	return true;
}

#endif //WITH_DEV_AUTOMATION_TESTS
