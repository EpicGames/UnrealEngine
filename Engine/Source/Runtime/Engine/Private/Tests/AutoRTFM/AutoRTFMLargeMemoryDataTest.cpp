// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/AutomationTest.h"
#include "AutoRTFM.h"
#include "Serialization/LargeMemoryData.h"

#if WITH_DEV_AUTOMATION_TESTS

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FAutoRTFMLargeMemoryDataTest, "AutoRTFM + FPooledLargeMemoryData", EAutomationTestFlags::EngineFilter | EAutomationTestFlags::ClientContext | EAutomationTestFlags::ServerContext | EAutomationTestFlags::CommandletContext)

bool FAutoRTFMLargeMemoryDataTest::RunTest(const FString & Parameters)
{
	if (!AutoRTFM::ForTheRuntime::IsAutoRTFMRuntimeEnabled())
	{
		ExecutionInfo.AddEvent(FAutomationEvent(EAutomationEventType::Info, TEXT("SKIPPED 'FAutoRTFMLargeMemoryDataTest' test. AutoRTFM disabled.")));
		return true;
	}

	AutoRTFM::Transact([&]
	{
		FPooledLargeMemoryData Data;
	});

	AutoRTFM::Transact([&]
	{
		FPooledLargeMemoryData Data;
		AutoRTFM::AbortTransaction();
	});

	AutoRTFM::Transact([&]
	{
		{
			FPooledLargeMemoryData Data;
		}
		AutoRTFM::AbortTransaction();
	});

	return true;
}

#endif //WITH_DEV_AUTOMATION_TESTS
