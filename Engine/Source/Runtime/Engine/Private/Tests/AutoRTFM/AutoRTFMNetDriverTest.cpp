// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/AutomationTest.h"
#include "AutoRTFM.h"
#include "Engine/ActorChannel.h"
#include "Engine/DemoNetDriver.h"

#if WITH_DEV_AUTOMATION_TESTS

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FAutoRTFMNetDriverTests, "AutoRTFM + FTraceFilter", EAutomationTestFlags::EngineFilter | EAutomationTestFlags::ClientContext | EAutomationTestFlags::ServerContext | EAutomationTestFlags::CommandletContext)

bool FAutoRTFMNetDriverTests::RunTest(const FString & Parameters)
{
	if (!AutoRTFM::ForTheRuntime::IsAutoRTFMRuntimeEnabled())
	{
		ExecutionInfo.AddEvent(FAutomationEvent(EAutomationEventType::Info, TEXT("SKIPPED 'FAutoRTFMNetDriverTests' test. AutoRTFM disabled.")));
		return true;
	}

	UNetDriver* const Driver = NewObject<UDemoNetDriver>();

	UNetConnection* const Connection = NewObject<UDemoNetConnection>();
	Connection->Driver = Driver;
	Driver->AddClientConnection(Connection);

	UActorChannel* const ActorChannel = NewObject<UActorChannel>();
	ActorChannel->OpenedLocally = true;
	ActorChannel->Connection = Connection;
	Connection->Channels.Add(ActorChannel);
	Connection->OpenChannels.Add(ActorChannel);

	FString Description;

	AutoRTFM::ETransactionResult Result = AutoRTFM::Transact([&]
		{
			Description = ActorChannel->Describe();
			AutoRTFM::AbortTransaction();
		});

	TestTrueExpr(AutoRTFM::ETransactionResult::AbortedByRequest == Result);
	TestTrueExpr(Description.IsEmpty());

	Result = AutoRTFM::Transact([&]
		{
			Description = ActorChannel->Describe();
		});

	TestTrueExpr(AutoRTFM::ETransactionResult::Committed == Result);
	TestFalseExpr(Description.IsEmpty());

	return true;
}

#endif //WITH_DEV_AUTOMATION_TESTS
