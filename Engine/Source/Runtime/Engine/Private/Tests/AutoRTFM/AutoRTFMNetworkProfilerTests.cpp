// Copyright Epic Games, Inc. All Rights Reserved.

#include "AutoRTFM/Testing.h"
#include "AutoRTFMTestActor.h"
#include "Engine/ActorChannel.h"
#include "Engine/DemoNetDriver.h"
#include "Engine/NetDriver.h"
#include "Misc/AutomationTest.h"
#include "AutoRTFM.h"
#include "Net/NetworkProfiler.h"
#include "UObject/UnrealType.h"

#if WITH_DEV_AUTOMATION_TESTS

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FAutoRTFMNetProfilerTests, "AutoRTFM + FNetworkProfiler", EAutomationTestFlags::EngineFilter | EAutomationTestFlags::ClientContext | EAutomationTestFlags::ServerContext | EAutomationTestFlags::CommandletContext)

bool FAutoRTFMNetProfilerTests::RunTest(const FString & Parameters)
{
	if (!AutoRTFM::ForTheRuntime::IsAutoRTFMRuntimeEnabled())
	{
		ExecutionInfo.AddEvent(FAutomationEvent(EAutomationEventType::Info, TEXT("SKIPPED 'FAutoRTFMNetProfilerTests' test. AutoRTFM disabled.")));
		return true;
	}

	UNetDriver* const Driver = NewObject<UDemoNetDriver>();

	UNetConnection* const Connection = NewObject<UDemoNetConnection>();
	Connection->Driver = Driver;
	Driver->AddClientConnection(Connection);

	UActorChannel* const ActorChannel = NewObject<UActorChannel>();
	ActorChannel->OpenedLocally = true;
	ActorChannel->Connection = Connection;
	Connection->Channels.Add(ActorChannel);
	Connection->OpenChannels.Add(ActorChannel);
	
	AAutoRTFMTestActor* const Actor = NewObject<AAutoRTFMTestActor>();

	FProperty* const Property = AAutoRTFMTestActor::StaticClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(AAutoRTFMTestActor, MyProperty));

	FNetworkProfiler Profiler;
	Profiler.EnableTracking(/* bShouldEnableTracking */ true);
	Profiler.TrackSessionChange(/* bShouldContinueTracking */ true, FURL(TEXT("FAutoRTFMNetProfilerTests")));

	AutoRTFM::Testing::Commit([&]
	{
		Profiler.TrackWritePropertyHandle(16, Connection);
	});
	AutoRTFM::Testing::Abort([&]
	{
		Profiler.TrackWritePropertyHandle(16, Connection);
		AutoRTFM::AbortTransaction();
	});

	AutoRTFM::Testing::Commit([&]
	{
		Profiler.TrackWritePropertyHeader(Property, 16, Connection);
	});
	AutoRTFM::Testing::Abort([&]
	{
		Profiler.TrackWritePropertyHeader(Property, 16, Connection);
		AutoRTFM::AbortTransaction();
	});

	AutoRTFM::Testing::Commit([&]
	{
		Profiler.TrackReplicateProperty(Property, 16, Connection);
	});
	AutoRTFM::Testing::Abort([&]
	{
		Profiler.TrackReplicateProperty(Property, 16, Connection);
		AutoRTFM::AbortTransaction();
	});

	AutoRTFM::Testing::Commit([&]
	{
		Profiler.TrackBeginContentBlock(Actor, 16, Connection);
	});
	AutoRTFM::Testing::Abort([&]
	{
		Profiler.TrackBeginContentBlock(Actor, 16, Connection);
		AutoRTFM::AbortTransaction();
	});

	AutoRTFM::Testing::Commit([&]
	{
		Profiler.TrackEndContentBlock(Actor, 16, Connection);
	});
	AutoRTFM::Testing::Abort([&]
	{
		Profiler.TrackEndContentBlock(Actor, 16, Connection);
		AutoRTFM::AbortTransaction();
	});

	return true;
}

#endif //WITH_DEV_AUTOMATION_TESTS
