// Copyright Epic Games, Inc. All Rights Reserved.

#include "AutoRTFM/Testing.h"
#include "Tests/TestHarnessAdapter.h"
#include "AutoRTFM.h"

#include "Framework/Text/CharRangeList.h"

#if WITH_DEV_AUTOMATION_TESTS

TEST_CASE_NAMED(FAutoRTFMSlateTests, "AutoRTFM.Slate", "[EngineFilter][ClientContext][ServerContext][CommandletContext][SupportsAutoRTFM]")
{
	// Test for SOL-7842
	FCharRangeList CharRangeList;
	TestTrueExpr(CharRangeList.IsEmpty());
	CharRangeList.InitializeFromString(TEXT("a-zA-Z0-9._"));
	TestFalseExpr(CharRangeList.IsEmpty());
}

#endif // WITH_DEV_AUTOMATION_TESTS
