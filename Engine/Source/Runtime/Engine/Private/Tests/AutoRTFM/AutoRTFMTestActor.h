// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GameFramework/Actor.h"
#include "Misc/MTTransactionallySafeAccessDetector.h"
#include "Misc/TransactionallySafeCriticalSection.h"
#include "Misc/TransactionallySafeRWLock.h"
#include "UObject/ObjectMacros.h"
#include "AutoRTFMTestActor.generated.h"

UCLASS()
class AAutoRTFMTestActor : public AActor
{
    GENERATED_BODY()

public:
	int Value = 42;

	UPROPERTY()
	int32 MyProperty = 0;

	// Various AutoRTFM-complex member fields, so that the tests exercise their
	// ctor / dtors in interesting ways.
	FTransactionallySafeCriticalSection CriticalSection;
	FTransactionallySafeRWLock RWLock;
	UE_MT_DECLARE_TS_RW_ACCESS_DETECTOR(RWAccessDetector)
};
