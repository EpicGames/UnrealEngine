// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GameFramework/Actor.h"
#include "AutoRTFMTestAnotherActor.generated.h"

UCLASS()
class AAutoRTFMTestAnotherActor : public AActor
{
    GENERATED_BODY()

public:
    int Value = 42;
};
