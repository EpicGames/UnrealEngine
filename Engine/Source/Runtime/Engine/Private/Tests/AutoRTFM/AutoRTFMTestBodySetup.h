// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PhysicsEngine/BodySetup.h"
#include "AutoRTFMTestBodySetup.generated.h"

UCLASS()
class UAutoRTFMTestBodySetup : public UBodySetup
{
    GENERATED_BODY()

public:
    int Value = 42;
};
