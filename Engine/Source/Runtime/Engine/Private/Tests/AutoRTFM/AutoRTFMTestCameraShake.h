// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Camera/CameraShakeBase.h"
#include "AutoRTFMTestCameraShake.generated.h"

UCLASS(NotBlueprintable, HideDropdown)
class UAutoRTFMTestCameraShake : public UCameraShakeBase
{
public:
	GENERATED_BODY()
};
