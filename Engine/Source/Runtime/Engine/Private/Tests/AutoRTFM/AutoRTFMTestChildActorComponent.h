// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Components/ChildActorComponent.h"
#include "AutoRTFMTestChildActorComponent.generated.h"

UCLASS()
class UAutoRTFMTestChildActorComponent : public UChildActorComponent
{
    GENERATED_BODY()

public:
	void ForceActorClass(UClass* const Class)
	{
		SetChildActorClass(Class);
	}
};
