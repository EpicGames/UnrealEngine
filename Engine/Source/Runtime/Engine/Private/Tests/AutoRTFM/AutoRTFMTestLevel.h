// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/Level.h"
#include "AutoRTFMTestLevel.generated.h"

UCLASS()
class UAutoRTFMTestLevel : public ULevel
{
    GENERATED_BODY()

public:
    int Value = 42;
};
