// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AutoRTFMTestObject.generated.h"

UCLASS()
class UAutoRTFMTestObject : public UObject
{
    GENERATED_BODY()

public:
    UAutoRTFMTestObject(const FObjectInitializer& ObjectInitializer = FObjectInitializer::Get()) : Value(42)
    {
        UObject* const Obj = ObjectInitializer.GetObj();
        UObject* const Outer = Obj->GetOuter();

        if (Outer->IsA<UAutoRTFMTestObject>())
        {
            UAutoRTFMTestObject* const OuterAsType = static_cast<UAutoRTFMTestObject*>(Outer);
            OuterAsType->Value += 13;
        }
    }

    int Value;

	bool bHitOnComponentPhysicsStateChanged = false;

	UFUNCTION()
	void OnComponentPhysicsStateChanged(UPrimitiveComponent* ChangedComponent, EComponentPhysicsStateChange StateChange)
	{
		bHitOnComponentPhysicsStateChanged = true;
	}
    
    void SetValueTo100()
    {
        Value = 100;
    }
};
