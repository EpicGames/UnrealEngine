// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Components/PrimitiveComponent.h"
#include "Misc/MTTransactionallySafeAccessDetector.h"
#include "Misc/TransactionallySafeCriticalSection.h"
#include "Misc/TransactionallySafeRWLock.h"
#include "AutoRTFMTestPrimitiveComponent.generated.h"

UCLASS()
class UAutoRTFMTestPrimitiveComponent : public UPrimitiveComponent
{
	GENERATED_BODY()

public:
	int Value = 42;

	UBodySetup* BodySetup = nullptr;

	UBodySetup* GetBodySetup() override { return BodySetup; }

	// Various AutoRTFM-complex member fields, so that the tests exercise their
	// ctor / dtors in interesting ways.
	FTransactionallySafeCriticalSection CriticalSection;
	FTransactionallySafeRWLock RWLock;
	UE_MT_DECLARE_TS_RW_ACCESS_DETECTOR(RWAccessDetector)
};
