// Copyright Epic Games, Inc. All Rights Reserved.

#include "AutoRTFM/Testing.h"
#include "Misc/AssertionMacros.h"

#if WITH_DEV_AUTOMATION_TESTS

void AutoRTFM::Testing::AssertionFailure(const char* Expression, const char* File, int Line)
{
	FDebug::AssertFailed(Expression, File, Line);
}

#endif
