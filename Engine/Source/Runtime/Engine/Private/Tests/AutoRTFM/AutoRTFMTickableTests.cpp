// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tests/TestHarnessAdapter.h"
#include "AutoRTFM.h"
#include "Tickable.h"

#if WITH_DEV_AUTOMATION_TESTS

TEST_CASE_NAMED(FAutoRTFMTickableTests, "AutoRTFM.FTickableObject", "[EngineFilter][ClientContext][ServerContext][CommandletContext][SupportsAutoRTFM]")
{
	struct FMyTickableGameObject final : FTickableGameObject
	{
		void Tick(float) override {}
		TStatId GetStatId() const override { return TStatId(); }
	};

	FMyTickableGameObject Tickable;
	Tickable.SetTickableTickType(ETickableTickType::Always);
}

#endif //WITH_DEV_AUTOMATION_TESTS
