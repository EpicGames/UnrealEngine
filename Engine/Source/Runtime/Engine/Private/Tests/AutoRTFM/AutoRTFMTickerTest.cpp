// Copyright Epic Games, Inc. All Rights Reserved.

#include "Containers/Ticker.h"
#include "Misc/AutomationTest.h"
#include "AutoRTFM.h"
#include <vector>

#if WITH_DEV_AUTOMATION_TESTS

#define CHECK_EQ(A, B) \
	UTEST_EQUAL(TEXT(__FILE__ ":" UE_STRINGIZE(__LINE__) ": UTEST_EQUAL_EXPR(" #A ", " #B ")"), A, B)

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FAutoRTFMTickerTest, "AutoRTFM + FTSTicker", EAutomationTestFlags::EngineFilter | EAutomationTestFlags::ClientContext | EAutomationTestFlags::ServerContext | EAutomationTestFlags::CommandletContext)

bool FAutoRTFMTickerTest::RunTest(const FString & Parameters)
{
	if (!AutoRTFM::ForTheRuntime::IsAutoRTFMRuntimeEnabled())
	{
		ExecutionInfo.AddEvent(FAutomationEvent(EAutomationEventType::Info, TEXT("SKIPPED 'FAutoRTFMTickerTest' test. AutoRTFM disabled.")));
		return true;
	}

	FTSTicker& Ticker = FTSTicker::GetCoreTicker();

	// A ticker delegate that appends the delta-time to CallbackEvents.
	std::vector<float> CallbackEvents;
	auto Delegate = [&](float DeltaTime)
	{ 
		AutoRTFM::Open([&]{ CallbackEvents.push_back(DeltaTime); });
		return /* reschedule? */ false;
	};

	auto Section = [&](const TCHAR* Name, auto&& Test)
	{
		bool Result = Test();
		CallbackEvents.clear();
		Ticker.Reset();
		if (!Result)
		{
			AddError(FString::Printf(TEXT("In section '%s'."), Name), 1);
		}
	};

	Section(TEXT("Basic assumptions"), [&]
	{
		Section(TEXT("Explicit remove"), [&]
		{
			auto Handle = Ticker.AddTicker(TEXT(""), /* Delay */ 0.1f, Delegate);
			FTSTicker::RemoveTicker(Handle);
			Ticker.Tick(1.0f);
			CHECK_EQ(CallbackEvents, {});
			return true;
		});
		Section(TEXT("One-shot self removal"), [&]
		{
			Ticker.AddTicker(TEXT(""), /* Delay */ 0.1f, Delegate);
			Ticker.Tick(1.0f);
			CHECK_EQ(CallbackEvents, {1.0f});
			Ticker.Tick(2.0f);
			CHECK_EQ(CallbackEvents, {1.0f}); // Doesn't repeat.
			return true;
		});
		return true;
	});

	Section(TEXT("Transact(AddTicker), Tick"), [&]
	{
		AutoRTFM::Transact([&]
		{
			Ticker.AddTicker(TEXT(""), /* Delay */ 0.1f, Delegate);
		});
		Ticker.Tick(1.0f);
		CHECK_EQ(CallbackEvents, {1.0f});
		return true;
	});

	Section(TEXT("Transact(AddTicker, Abort), Tick"), [&]
	{
		AutoRTFM::Transact([&]
		{
			Ticker.AddTicker(TEXT(""), /* Delay */ 0.1f, Delegate);
			AutoRTFM::AbortTransaction();
		});
		Ticker.Tick(1.0f);
		CHECK_EQ(CallbackEvents, {});
		return true;
	});

	Section(TEXT("Transact(AddTicker, RemoveTicker), Tick"), [&]
	{
		AutoRTFM::Transact([&]
		{
			FTSTicker::FDelegateHandle Handle = Ticker.AddTicker(TEXT(""), /* Delay */ 0.1f, Delegate);
			FTSTicker::RemoveTicker(Handle);
		});
		Ticker.Tick(1.0f);
		CHECK_EQ(CallbackEvents, {});
		return true;
	});

	Section(TEXT("Transact(AddTicker, RemoveTicker, Abort), Tick"), [&]
	{
		AutoRTFM::Transact([&]
		{
			FTSTicker::FDelegateHandle Handle = Ticker.AddTicker(TEXT(""), /* Delay */ 0.1f, Delegate);
			FTSTicker::RemoveTicker(Handle);
			AutoRTFM::AbortTransaction();
		});
		Ticker.Tick(1.0f);
		CHECK_EQ(CallbackEvents, {});
		return true;
	});

	Section(TEXT("Transact(AddTicker, Abort), Transact(AddTicker), Tick"), [&]
	{
		AutoRTFM::Transact([&]
		{
			Ticker.AddTicker(TEXT(""), /* Delay */ 0.1f, Delegate);
			AutoRTFM::AbortTransaction();
		});
		AutoRTFM::Transact([&]
		{
			Ticker.AddTicker(TEXT(""), /* Delay */ 0.1f, Delegate);
		});
		Ticker.Tick(1.0f);
		CHECK_EQ(CallbackEvents, {1.0f});
		return true;
	});

	Section(TEXT("AddTicker, Transact(RemoveTicker), Tick"), [&]
	{
		FTSTicker::FDelegateHandle Handle = Ticker.AddTicker(TEXT(""), /* Delay */ 0.1f, Delegate);

		AutoRTFM::Transact([&]
		{
			FTSTicker::RemoveTicker(Handle);
		});

		Ticker.Tick(1.0f);
		CHECK_EQ(CallbackEvents, {});
		return true;
	});

	Section(TEXT("AddTicker, Transact(RemoveTicker, Abort), Tick"), [&]
	{
		FTSTicker::FDelegateHandle Handle = Ticker.AddTicker(TEXT(""), /* Delay */ 0.1f, Delegate);

		AutoRTFM::Transact([&]
		{
			FTSTicker::RemoveTicker(Handle);
			AutoRTFM::AbortTransaction();
		});

		Ticker.Tick(1.0f);
		CHECK_EQ(CallbackEvents, {1.0f});
		return true;
	});

	return true;
}

#undef CHECK_EQ

#endif //WITH_DEV_AUTOMATION_TESTS
