// Copyright Epic Games, Inc. All Rights Reserved.

#include "AutoRTFM.h"
#include "AutoRTFM/Testing.h"
#include "AutoRTFMTestObject.h"
#include "CoreGlobals.h"
#include "Engine/TimerHandle.h"
#include "Misc/AssertionMacros.h"
#include "Misc/AutomationTest.h"
#include "TimerManager.h"

#if WITH_DEV_AUTOMATION_TESTS

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FAutoRTFMTimerManagerTest, "AutoRTFM + FTimerManager", EAutomationTestFlags::EngineFilter | EAutomationTestFlags::ClientContext | EAutomationTestFlags::ServerContext | EAutomationTestFlags::CommandletContext)

bool FAutoRTFMTimerManagerTest::RunTest(const FString & Parameters)
{
	if (!AutoRTFM::ForTheRuntime::IsAutoRTFMRuntimeEnabled())
	{
		ExecutionInfo.AddEvent(FAutomationEvent(EAutomationEventType::Info, TEXT("SKIPPED 'FAutoRTFMTimerManagerTest' test. AutoRTFM disabled.")));
		return true;
	}

	auto Section = [&](const TCHAR* Name, auto&& Test)
	{
		{
			FTimerManager TimerManager;
			TimerManager.Tick(0);
			if (!Test(TimerManager))
			{
				AddError(FString::Printf(TEXT("In section '%s' (HasBeenTickedThisFrame: true)."), Name), 1);
			}
		}
		{
			FTimerManager TimerManager;
			TimerManager.Tick(0);
			GFrameCounter++;
			if (!Test(TimerManager))
			{
				AddError(FString::Printf(TEXT("In section '%s' (HasBeenTickedThisFrame: false)."), Name), 1);
			}
		}
	};
	
	auto Tick = [&](FTimerManager& TimerManager)
	{
		// Simulate two frames to ensure that pending timers are activated and triggered.
		for (int I = 0; I < 2; I++)
		{
			GFrameCounter++;
			TimerManager.Tick(1.5f);
		}
	};

	// Wrapper around FTimerManager::ClearTimer() to prevent invalidating of the handle argument.
	auto ClearTimer = [&] AUTORTFM_ENABLE (FTimerManager& TimerManager, FTimerHandle TimerHandle)
	{
		TimerManager.ClearTimer(TimerHandle);
	};

	struct AUTORTFM_ENABLE FTimer
	{
		FTimerHandle Handle;
		bool bCalled = false;
		TFunction<void ()> Callback()
		{
			return [this]
			{
				bCalled = true;
			};
		}
		void MarkCalled()
		{
			bCalled = true;
		}
	};

	Section(TEXT("Transact(SetTimer), Tick"), [&] (FTimerManager& TimerManager)
	{
		constexpr size_t NumTimers = 16;
		FTimer Timers[NumTimers]{};

		AutoRTFM::Testing::Commit([&]
		{
			for (FTimer& Timer : Timers)
			{
				TimerManager.SetTimer(Timer.Handle, Timer.Callback(), /* InRate */ 1.0f, FTimerManagerTimerParameters{});
				check(TimerManager.IsTimerActive(Timer.Handle));
				check(TimerManager.GetTimerRemaining(Timer.Handle) == 1.0f);
				check(TimerManager.GetTimerElapsed(Timer.Handle) == 0.0f);
			}
		});

		Tick(TimerManager);

		for (FTimer& Timer : Timers)
		{
			check(Timer.bCalled);
			check(TimerManager.GetTimerRemaining(Timer.Handle) == -1.0f);
			check(TimerManager.GetTimerElapsed(Timer.Handle) == -1.0f);
		}

		return true;
	});

	Section(TEXT("Transact(SetTimer, Abort), Tick"), [&] (FTimerManager& TimerManager)
	{
		constexpr size_t NumTimers = 16;
		FTimer Timers[NumTimers]{};

		AutoRTFM::Testing::Abort([&]
		{
			for (FTimer& Timer : Timers)
			{
				TimerManager.SetTimer(Timer.Handle, Timer.Callback(), /* InRate */ 1.0f, FTimerManagerTimerParameters{});
				check(TimerManager.IsTimerActive(Timer.Handle));
				check(TimerManager.GetTimerRemaining(Timer.Handle) == 1.0f);
				check(TimerManager.GetTimerElapsed(Timer.Handle) == 0.0f);
			}
			AutoRTFM::AbortTransaction();
		});

		Tick(TimerManager);

		for (FTimer& Timer : Timers)
		{
			check(!Timer.bCalled);
			check(!TimerManager.IsTimerActive(Timer.Handle));
		}

		return true;
	});

	Section(TEXT("Tick, Transact(SetTimer, Abort), Tick"), [&] (FTimerManager& TimerManager)
	{
		constexpr size_t NumTimers = 16;
		FTimer Timers[NumTimers]{};

		Tick(TimerManager);

		AutoRTFM::Testing::Abort([&]
		{
			for (FTimer& Timer : Timers)
			{
				TimerManager.SetTimer(Timer.Handle, Timer.Callback(), /* InRate */ 1.0f, FTimerManagerTimerParameters{});
				check(TimerManager.IsTimerActive(Timer.Handle));
				check(TimerManager.GetTimerRemaining(Timer.Handle) == 1.0f);
				check(TimerManager.GetTimerElapsed(Timer.Handle) == 0.0f);
			}
			AutoRTFM::AbortTransaction();
		});

		Tick(TimerManager);

		for (FTimer& Timer : Timers)
		{
			check(!Timer.bCalled);
			check(!TimerManager.IsTimerActive(Timer.Handle));
		}

		return true;
	});

	Section(TEXT("Transact(SetTimer, ClearTimer), Tick"), [&] (FTimerManager& TimerManager)
	{
		constexpr size_t NumTimers = 16;
		FTimer Timers[NumTimers]{};

		AutoRTFM::Testing::Commit([&]
		{
			for (FTimer& Timer : Timers)
			{
				TimerManager.SetTimer(Timer.Handle, Timer.Callback(), /* InRate */ 1.0f, FTimerManagerTimerParameters{});
				check(TimerManager.IsTimerActive(Timer.Handle));
				check(TimerManager.GetTimerRemaining(Timer.Handle) == 1.0f);
				check(TimerManager.GetTimerElapsed(Timer.Handle) == 0.0f);
				ClearTimer(TimerManager, Timer.Handle);
				check(!TimerManager.IsTimerActive(Timer.Handle));
				check(TimerManager.GetTimerRemaining(Timer.Handle) == -1.0f);
				check(TimerManager.GetTimerElapsed(Timer.Handle) == -1.0f);
			}
		});

		Tick(TimerManager);

		for (FTimer& Timer : Timers)
		{
			check(!Timer.bCalled);
			check(!TimerManager.IsTimerActive(Timer.Handle));
			check(TimerManager.GetTimerRemaining(Timer.Handle) == -1.0f);
			check(TimerManager.GetTimerElapsed(Timer.Handle) == -1.0f);
		}

		return true;
	});

	Section(TEXT("Transact(SetTimer, ClearTimer, Abort), Tick"), [&] (FTimerManager& TimerManager)
	{
		constexpr size_t NumTimers = 16;
		FTimer Timers[NumTimers]{};

		AutoRTFM::Testing::Abort([&]
		{
			for (FTimer& Timer : Timers)
			{
				TimerManager.SetTimer(Timer.Handle, Timer.Callback(), /* InRate */ 1.0f, FTimerManagerTimerParameters{});
				check(TimerManager.IsTimerActive(Timer.Handle));
				check(TimerManager.GetTimerRemaining(Timer.Handle) == 1.0f);
				check(TimerManager.GetTimerElapsed(Timer.Handle) == 0.0f);
				ClearTimer(TimerManager, Timer.Handle);
				check(!TimerManager.IsTimerActive(Timer.Handle));
				check(TimerManager.GetTimerRemaining(Timer.Handle) == -1.0f);
				check(TimerManager.GetTimerElapsed(Timer.Handle) == -1.0f);
			}
			AutoRTFM::AbortTransaction();
		});

		Tick(TimerManager);

		for (FTimer& Timer : Timers)
		{
			check(!Timer.bCalled);
			check(!TimerManager.IsTimerActive(Timer.Handle));
			check(TimerManager.GetTimerRemaining(Timer.Handle) == -1.0f);
			check(TimerManager.GetTimerElapsed(Timer.Handle) == -1.0f);
		}

		return true;
	});

	Section(TEXT("Transact(SetTimer, Open(SetTimer)), Tick"), [&] (FTimerManager& TimerManager)
	{
		constexpr size_t NumOpenTimers = 16;
		FTimer OpenTimers[NumOpenTimers];
		constexpr size_t NumClosedTimers = 16;
		FTimer ClosedTimers[NumClosedTimers];

		AutoRTFM::Testing::Commit([&]
		{
			for (FTimer& ClosedTimer : ClosedTimers)
			{
				TimerManager.SetTimer(ClosedTimer.Handle, ClosedTimer.Callback(), /* InRate */ 1.0f, FTimerManagerTimerParameters{});
				check(TimerManager.IsTimerActive(ClosedTimer.Handle));
				check(TimerManager.GetTimerRemaining(ClosedTimer.Handle) == 1.0f);
				check(TimerManager.GetTimerElapsed(ClosedTimer.Handle) == 0.0f);
			}
			
			AutoRTFM::Open([&]
			{
				for (FTimer& OpenTimer : OpenTimers)
				{
					TimerManager.SetTimer(OpenTimer.Handle, OpenTimer.Callback(), /* InRate */ 1.0f, FTimerManagerTimerParameters{});
					check(TimerManager.IsTimerActive(OpenTimer.Handle));
					check(TimerManager.GetTimerRemaining(OpenTimer.Handle) == 1.0f);
					check(TimerManager.GetTimerElapsed(OpenTimer.Handle) == 0.0f);
				}
			});
		});

		Tick(TimerManager);

		for (FTimer& ClosedTimer : ClosedTimers)
		{
			check(ClosedTimer.bCalled);
			check(TimerManager.GetTimerRemaining(ClosedTimer.Handle) == -1.0f);
			check(TimerManager.GetTimerElapsed(ClosedTimer.Handle) == -1.0f);
		}

		for (FTimer& OpenTimer : OpenTimers)
		{
			check(OpenTimer.bCalled);
			check(TimerManager.GetTimerRemaining(OpenTimer.Handle) == -1.0f);
			check(TimerManager.GetTimerElapsed(OpenTimer.Handle) == -1.0f);
		}

		return true;
	});

	Section(TEXT("Transact(SetTimer, Open(SetTimer), Abort), Tick"), [&] (FTimerManager& TimerManager)
	{
		constexpr size_t NumOpenTimers = 16;
		FTimer OpenTimers[NumOpenTimers];
		constexpr size_t NumClosedTimers = 16;
		FTimer ClosedTimers[NumClosedTimers];

		AutoRTFM::Testing::Abort([&]
		{
			for (FTimer& ClosedTimer : ClosedTimers)
			{
				TimerManager.SetTimer(ClosedTimer.Handle, ClosedTimer.Callback(), /* InRate */ 1.0f, FTimerManagerTimerParameters{});
				check(TimerManager.IsTimerActive(ClosedTimer.Handle));
				check(TimerManager.GetTimerRemaining(ClosedTimer.Handle) == 1.0f);
				check(TimerManager.GetTimerElapsed(ClosedTimer.Handle) == 0.0f);
			}
			
			AutoRTFM::Open([&]
			{
				for (FTimer& OpenTimer : OpenTimers)
				{
					TimerManager.SetTimer(OpenTimer.Handle, OpenTimer.Callback(), /* InRate */ 1.0f, FTimerManagerTimerParameters{});
					check(TimerManager.IsTimerActive(OpenTimer.Handle));
					check(TimerManager.GetTimerRemaining(OpenTimer.Handle) == 1.0f);
					check(TimerManager.GetTimerElapsed(OpenTimer.Handle) == 0.0f);
				}
			});
			
			AutoRTFM::AbortTransaction();
		});

		Tick(TimerManager);

		for (FTimer& ClosedTimer : ClosedTimers)
		{
			check(!ClosedTimer.bCalled);
			check(TimerManager.GetTimerRemaining(ClosedTimer.Handle) == -1.0f);
			check(TimerManager.GetTimerElapsed(ClosedTimer.Handle) == -1.0f);
		}

		for (FTimer& OpenTimer : OpenTimers)
		{
			check(OpenTimer.bCalled);
			check(TimerManager.GetTimerRemaining(OpenTimer.Handle) == -1.0f);
			check(TimerManager.GetTimerElapsed(OpenTimer.Handle) == -1.0f);
		}

		return true;
	});

	Section(TEXT("Transact(Open(SetTimer), SetTimer, Open(ClearTimer)), Tick"), [&] (FTimerManager& TimerManager)
	{
		constexpr size_t NumOpenTimers = 16;
		FTimer OpenTimers[NumOpenTimers];
		constexpr size_t NumClosedTimers = 16;
		FTimer ClosedTimers[NumClosedTimers];

		AutoRTFM::Testing::Commit([&]
		{
			AutoRTFM::Open([&]
			{
				for (FTimer& OpenTimer : OpenTimers)
				{
					TimerManager.SetTimer(OpenTimer.Handle, OpenTimer.Callback(), /* InRate */ 1.0f, FTimerManagerTimerParameters{});
					check(TimerManager.IsTimerActive(OpenTimer.Handle));
					check(TimerManager.GetTimerRemaining(OpenTimer.Handle) == 1.0f);
					check(TimerManager.GetTimerElapsed(OpenTimer.Handle) == 0.0f);
				}
			});

			for (FTimer& ClosedTimer : ClosedTimers)
			{
				TimerManager.SetTimer(ClosedTimer.Handle, ClosedTimer.Callback(), /* InRate */ 1.0f, FTimerManagerTimerParameters{});
				check(TimerManager.IsTimerActive(ClosedTimer.Handle));
				check(TimerManager.GetTimerRemaining(ClosedTimer.Handle) == 1.0f);
				check(TimerManager.GetTimerElapsed(ClosedTimer.Handle) == 0.0f);
			}
			
			AutoRTFM::Open([&]
			{
				for (FTimer& OpenTimer : OpenTimers)
				{
					check(TimerManager.IsTimerActive(OpenTimer.Handle));
					ClearTimer(TimerManager, OpenTimer.Handle);
					check(!TimerManager.IsTimerActive(OpenTimer.Handle));
					check(TimerManager.GetTimerRemaining(OpenTimer.Handle) == -1.0f);
					check(TimerManager.GetTimerElapsed(OpenTimer.Handle) == -1.0f);
				}
			});
		});

		Tick(TimerManager);

		for (FTimer& OpenTimer : OpenTimers)
		{
			check(!OpenTimer.bCalled);
			check(TimerManager.GetTimerRemaining(OpenTimer.Handle) == -1.0f);
			check(TimerManager.GetTimerElapsed(OpenTimer.Handle) == -1.0f);
		}

		for (FTimer& ClosedTimer : ClosedTimers)
		{
			check(ClosedTimer.bCalled);
			check(TimerManager.GetTimerRemaining(ClosedTimer.Handle) == -1.0f);
			check(TimerManager.GetTimerElapsed(ClosedTimer.Handle) == -1.0f);
		}

		return true;
	});

	Section(TEXT("Transact(Open(SetTimer), SetTimer, Open(ClearTimer), Abort), Tick"), [&] (FTimerManager& TimerManager)
	{
		constexpr size_t NumOpenTimers = 16;
		FTimer OpenTimers[NumOpenTimers];
		constexpr size_t NumClosedTimers = 16;
		FTimer ClosedTimers[NumClosedTimers];

		AutoRTFM::Testing::Abort([&]
		{
			AutoRTFM::Open([&]
			{
				for (FTimer& OpenTimer : OpenTimers)
				{
					TimerManager.SetTimer(OpenTimer.Handle, OpenTimer.Callback(), /* InRate */ 1.0f, FTimerManagerTimerParameters{});
					check(TimerManager.IsTimerActive(OpenTimer.Handle));
					check(TimerManager.GetTimerRemaining(OpenTimer.Handle) == 1.0f);
					check(TimerManager.GetTimerElapsed(OpenTimer.Handle) == 0.0f);
				}
			});

			for (FTimer& ClosedTimer : ClosedTimers)
			{
				TimerManager.SetTimer(ClosedTimer.Handle, ClosedTimer.Callback(), /* InRate */ 1.0f, FTimerManagerTimerParameters{});
				check(TimerManager.IsTimerActive(ClosedTimer.Handle));
				check(TimerManager.GetTimerRemaining(ClosedTimer.Handle) == 1.0f);
				check(TimerManager.GetTimerElapsed(ClosedTimer.Handle) == 0.0f);
			}
			
			AutoRTFM::Open([&]
			{
				for (FTimer& OpenTimer : OpenTimers)
				{
					check(TimerManager.IsTimerActive(OpenTimer.Handle));
					ClearTimer(TimerManager, OpenTimer.Handle);
					check(!TimerManager.IsTimerActive(OpenTimer.Handle));
					check(TimerManager.GetTimerRemaining(OpenTimer.Handle) == -1.0f);
					check(TimerManager.GetTimerElapsed(OpenTimer.Handle) == -1.0f);
				}
			});
			
			AutoRTFM::AbortTransaction();
		});

		Tick(TimerManager);

		for (FTimer& OpenTimer : OpenTimers)
		{
			check(!OpenTimer.bCalled);
			check(TimerManager.GetTimerRemaining(OpenTimer.Handle) == -1.0f);
			check(TimerManager.GetTimerElapsed(OpenTimer.Handle) == -1.0f);
		}

		for (FTimer& ClosedTimer : ClosedTimers)
		{
			check(!ClosedTimer.bCalled);
			check(TimerManager.GetTimerRemaining(ClosedTimer.Handle) == -1.0f);
			check(TimerManager.GetTimerElapsed(ClosedTimer.Handle) == -1.0f);
		}

		return true;
	});

	Section(TEXT("SetTimer, Transact(ClearTimer), Tick"), [&] (FTimerManager& TimerManager)
	{
		constexpr size_t NumTimers = 16;
		FTimer Timers[NumTimers]{};

		for (FTimer& Timer : Timers)
		{
			TimerManager.SetTimer(Timer.Handle, Timer.Callback(), /* InRate */ 1.0f, FTimerManagerTimerParameters{});
			check(TimerManager.IsTimerActive(Timer.Handle));
			check(TimerManager.GetTimerRemaining(Timer.Handle) == 1.0f);
			check(TimerManager.GetTimerElapsed(Timer.Handle) == 0.0f);
		}

		AutoRTFM::Testing::Commit([&]
		{
			for (size_t I = 0; I < NumTimers; I++)
			{
				if ((I & 1) == 0)
				{
					FTimer& Timer = Timers[I];
					check(TimerManager.IsTimerActive(Timer.Handle));
					ClearTimer(TimerManager, Timer.Handle);
					check(!TimerManager.IsTimerActive(Timer.Handle));
					check(TimerManager.GetTimerRemaining(Timer.Handle) == -1.0f);
					check(TimerManager.GetTimerElapsed(Timer.Handle) == -1.0f);
				}
			}
		});

		Tick(TimerManager);

		for (size_t I = 0; I < NumTimers; I++)
		{
			FTimer& Timer = Timers[I];
			bool const bCleared = (I & 1) == 0;
			check(Timer.bCalled == !bCleared);
			check(TimerManager.GetTimerRemaining(Timer.Handle) == -1.0f);
			check(TimerManager.GetTimerElapsed(Timer.Handle) == -1.0f);
		}

		return true;
	});

	Section(TEXT("SetTimer, Transact(ClearTimer, Abort), Tick"), [&] (FTimerManager& TimerManager)
	{
		constexpr size_t NumTimers = 16;
		FTimer Timers[NumTimers]{};

		for (FTimer& Timer : Timers)
		{
			TimerManager.SetTimer(Timer.Handle, Timer.Callback(), /* InRate */ 1.0f, FTimerManagerTimerParameters{});
			check(TimerManager.IsTimerActive(Timer.Handle));
			check(TimerManager.GetTimerRemaining(Timer.Handle) == 1.0f);
			check(TimerManager.GetTimerElapsed(Timer.Handle) == 0.0f);
		}

		AutoRTFM::Testing::Abort([&]
		{
			for (size_t I = 0; I < NumTimers; I++)
			{
				FTimer& Timer = Timers[I];
				ClearTimer(TimerManager, Timer.Handle);
				check(TimerManager.GetTimerRemaining(Timer.Handle) == -1.0f);
				check(TimerManager.GetTimerElapsed(Timer.Handle) == -1.0f);
			}
			AutoRTFM::AbortTransaction();
		});

		for (size_t I = 0; I < NumTimers; I++)
		{
			FTimer& Timer = Timers[I];
			check(TimerManager.GetTimerRemaining(Timer.Handle) == 1.0f);
			check(TimerManager.GetTimerElapsed(Timer.Handle) == 0.0f);
		}

		Tick(TimerManager);

		for (size_t I = 0; I < NumTimers; I++)
		{
			FTimer& Timer = Timers[I];
			check(Timer.bCalled);
			check(TimerManager.GetTimerRemaining(Timer.Handle) == -1.0f);
			check(TimerManager.GetTimerElapsed(Timer.Handle) == -1.0f);
		}

		return true;
	});

	Section(TEXT("SetTimer, Transact(PauseTimer), Tick"), [&] (FTimerManager& TimerManager)
	{
		constexpr size_t NumTimers = 16;
		FTimer Timers[NumTimers]{};

		for (FTimer& Timer : Timers)
		{
			TimerManager.SetTimer(Timer.Handle, Timer.Callback(), /* InRate */ 1.0f, FTimerManagerTimerParameters{});
			check(TimerManager.IsTimerActive(Timer.Handle));
			check(TimerManager.GetTimerRemaining(Timer.Handle) == 1.0f);
			check(TimerManager.GetTimerElapsed(Timer.Handle) == 0.0f);
		}

		AutoRTFM::Testing::Commit([&]
		{
			for (size_t I = 0; I < NumTimers; I++)
			{
				if ((I & 1) == 0)
				{
					FTimer& Timer = Timers[I];
					check(!TimerManager.IsTimerPaused(Timer.Handle));
					check(TimerManager.GetTimerRemaining(Timer.Handle) == 1.0f);
					check(TimerManager.GetTimerElapsed(Timer.Handle) == 0.0f);
					TimerManager.PauseTimer(Timer.Handle);
					check(TimerManager.IsTimerPaused(Timer.Handle));
					check(TimerManager.GetTimerRemaining(Timer.Handle) == 1.0f);
					check(TimerManager.GetTimerElapsed(Timer.Handle) == 0.0f);
				}
			}
		});

		Tick(TimerManager);

		for (size_t I = 0; I < NumTimers; I++)
		{
			FTimer& Timer = Timers[I];
			bool const bPaused = (I & 1) == 0;
			check(Timer.bCalled == !bPaused);
			check(TimerManager.IsTimerPaused(Timer.Handle) == bPaused);
			check(TimerManager.GetTimerRemaining(Timer.Handle) == (bPaused ? 1.0f : -1.0f));
			check(TimerManager.GetTimerElapsed(Timer.Handle) == (bPaused ? 0.0f : -1.0f));
		}

		return true;
	});

	Section(TEXT("SetTimer, Transact(PauseTimer, Abort), Tick"), [&] (FTimerManager& TimerManager)
	{
		constexpr size_t NumTimers = 16;
		FTimer Timers[NumTimers]{};

		for (FTimer& Timer : Timers)
		{
			TimerManager.SetTimer(Timer.Handle, Timer.Callback(), /* InRate */ 1.0f, FTimerManagerTimerParameters{});
			check(TimerManager.IsTimerActive(Timer.Handle));
			check(TimerManager.GetTimerRemaining(Timer.Handle) == 1.0f);
			check(TimerManager.GetTimerElapsed(Timer.Handle) == 0.0f);
		}

		AutoRTFM::Testing::Abort([&]
		{
			for (size_t I = 0; I < NumTimers; I++)
			{
				if ((I & 1) == 0)
				{
					FTimer& Timer = Timers[I];
					TimerManager.PauseTimer(Timer.Handle);
					check(TimerManager.IsTimerPaused(Timer.Handle));
					check(TimerManager.GetTimerRemaining(Timer.Handle) == 1.0f);
					check(TimerManager.GetTimerElapsed(Timer.Handle) == 0.0f);
				}
			}
			AutoRTFM::AbortTransaction();
		});

		Tick(TimerManager);

		for (size_t I = 0; I < NumTimers; I++)
		{
			FTimer& Timer = Timers[I];
			check(Timer.bCalled);
			check(!TimerManager.IsTimerPaused(Timer.Handle));
			check(TimerManager.GetTimerRemaining(Timer.Handle) == -1.0f);
			check(TimerManager.GetTimerElapsed(Timer.Handle) == -1.0f);
		}

		return true;
	});

	Section(TEXT("SetTimer, PauseTimer, Transact(UnPauseTimer), Tick"), [&] (FTimerManager& TimerManager)
	{
		constexpr size_t NumTimers = 16;
		FTimer Timers[NumTimers]{};

		for (FTimer& Timer : Timers)
		{
			TimerManager.SetTimer(Timer.Handle, Timer.Callback(), /* InRate */ 1.0f, FTimerManagerTimerParameters{});
			check(TimerManager.IsTimerActive(Timer.Handle));
			check(TimerManager.GetTimerRemaining(Timer.Handle) == 1.0f);
			check(TimerManager.GetTimerElapsed(Timer.Handle) == 0.0f);
			TimerManager.PauseTimer(Timer.Handle);
			check(TimerManager.IsTimerPaused(Timer.Handle));
			check(TimerManager.GetTimerRemaining(Timer.Handle) == 1.0f);
			check(TimerManager.GetTimerElapsed(Timer.Handle) == 0.0f);
		}

		AutoRTFM::Testing::Commit([&]
		{
			for (size_t I = 0; I < NumTimers; I++)
			{
				if ((I & 1) == 0)
				{
					FTimer& Timer = Timers[I];
					check(TimerManager.IsTimerPaused(Timer.Handle));
					check(TimerManager.GetTimerRemaining(Timer.Handle) == 1.0f);
					check(TimerManager.GetTimerElapsed(Timer.Handle) == 0.0f);
					TimerManager.UnPauseTimer(Timer.Handle);
					check(!TimerManager.IsTimerPaused(Timer.Handle));
					check(TimerManager.GetTimerRemaining(Timer.Handle) == 1.0f);
					check(TimerManager.GetTimerElapsed(Timer.Handle) == 0.0f);
				}
			}
		});

		Tick(TimerManager);

		for (size_t I = 0; I < NumTimers; I++)
		{
			FTimer& Timer = Timers[I];
			bool const bPaused = (I & 1) != 0;
			check(Timer.bCalled == !bPaused);
			check(TimerManager.IsTimerPaused(Timer.Handle) == bPaused);
			check(TimerManager.GetTimerRemaining(Timer.Handle) == (bPaused ? 1.0f : -1.0f));
			check(TimerManager.GetTimerElapsed(Timer.Handle) == (bPaused ? 0.0f : -1.0f));
		}

		return true;
	});

	Section(TEXT("SetTimer, PauseTimer, Transact(UnPauseTimer, Abort), Tick"), [&] (FTimerManager& TimerManager)
	{
		constexpr size_t NumTimers = 16;
		FTimer Timers[NumTimers]{};

		for (FTimer& Timer : Timers)
		{
			TimerManager.SetTimer(Timer.Handle, Timer.Callback(), /* InRate */ 1.0f, FTimerManagerTimerParameters{});
			check(TimerManager.IsTimerActive(Timer.Handle));
			check(TimerManager.GetTimerRemaining(Timer.Handle) == 1.0f);
			check(TimerManager.GetTimerElapsed(Timer.Handle) == 0.0f);
			TimerManager.PauseTimer(Timer.Handle);
			check(TimerManager.IsTimerPaused(Timer.Handle));
			check(TimerManager.GetTimerRemaining(Timer.Handle) == 1.0f);
			check(TimerManager.GetTimerElapsed(Timer.Handle) == 0.0f);
		}

		AutoRTFM::Testing::Abort([&]
		{
			for (size_t I = 0; I < NumTimers; I++)
			{
				if ((I & 1) == 0)
				{
					FTimer& Timer = Timers[I];
					check(TimerManager.IsTimerPaused(Timer.Handle));
					check(TimerManager.GetTimerRemaining(Timer.Handle) == 1.0f);
					check(TimerManager.GetTimerElapsed(Timer.Handle) == 0.0f);
					TimerManager.UnPauseTimer(Timer.Handle);
					check(!TimerManager.IsTimerPaused(Timer.Handle));
					check(TimerManager.GetTimerRemaining(Timer.Handle) == 1.0f);
					check(TimerManager.GetTimerElapsed(Timer.Handle) == 0.0f);
				}
			}
			AutoRTFM::AbortTransaction();
		});

		Tick(TimerManager);

		for (size_t I = 0; I < NumTimers; I++)
		{
			FTimer& Timer = Timers[I];
			check(!Timer.bCalled);
			check(TimerManager.IsTimerPaused(Timer.Handle));
			check(TimerManager.GetTimerRemaining(Timer.Handle) == 1.0f);
			check(TimerManager.GetTimerElapsed(Timer.Handle) == 0.0f);
		}

		return true;
	});

	Section(TEXT("Transact(SetTimer, ClearTimer, Open(SetTimer), Abort), Tick"), [&] (FTimerManager& TimerManager)
	{
		constexpr size_t NumTimers = 16;
		FTimer ClosedTimers[NumTimers]{};
		FTimer OpenTimers[NumTimers]{};

		AutoRTFM::Testing::Abort([&]
		{
			for (FTimer& Timer : ClosedTimers)
			{
				TimerManager.SetTimer(Timer.Handle, Timer.Callback(), /* InRate */ 1.0f, FTimerManagerTimerParameters{});
				check(TimerManager.IsTimerActive(Timer.Handle));
				check(TimerManager.GetTimerRemaining(Timer.Handle) == 1.0f);
				check(TimerManager.GetTimerElapsed(Timer.Handle) == 0.0f);
			}

			for (FTimer& Timer : ClosedTimers)
			{
				check(TimerManager.IsTimerActive(Timer.Handle));
				ClearTimer(TimerManager, Timer.Handle);
				check(!TimerManager.IsTimerActive(Timer.Handle));
				check(TimerManager.GetTimerRemaining(Timer.Handle) == -1.0f);
				check(TimerManager.GetTimerElapsed(Timer.Handle) == -1.0f);
			}

			AutoRTFM::Open([&]
			{
				for (FTimer& Timer : OpenTimers)
				{
					TimerManager.SetTimer(Timer.Handle, Timer.Callback(), /* InRate */ 1.0f, FTimerManagerTimerParameters{});
					check(TimerManager.GetTimerRemaining(Timer.Handle) == 1.0f);
					check(TimerManager.GetTimerElapsed(Timer.Handle) == 0.0f);
				}
			});

			AutoRTFM::AbortTransaction();
		});

		Tick(TimerManager);

		for (FTimer& Timer : ClosedTimers)
		{
			check(!Timer.bCalled);
			check(TimerManager.GetTimerRemaining(Timer.Handle) == -1.0f);
			check(TimerManager.GetTimerElapsed(Timer.Handle) == -1.0f);
		}
		for (FTimer& Timer : OpenTimers)
		{
			check(Timer.bCalled);
			check(TimerManager.GetTimerRemaining(Timer.Handle) == -1.0f);
			check(TimerManager.GetTimerElapsed(Timer.Handle) == -1.0f);
		}

		return true;
	});

	Section(TEXT("SetTimerForNextTick, Transact(ClearAllTimersForObject), Tick"), [&] (FTimerManager& TimerManager)
	{
		UAutoRTFMTestObject* Object = NewObject<UAutoRTFMTestObject>();
		FTimerHandle TimerHandle = TimerManager.SetTimerForNextTick(Object, &UAutoRTFMTestObject::SetValueTo100);

		AutoRTFM::Testing::Commit([&]
		{
			TimerManager.ClearAllTimersForObject(Object);
			check(!TimerManager.IsTimerActive(TimerHandle));
		});

		Tick(TimerManager);

		check(Object->Value != 100);

		return true;
	});

	Section(TEXT("SetTimerForNextTick, Transact(ClearAllTimersForObject, Abort), Tick"), [&] (FTimerManager& TimerManager)
	{
		UAutoRTFMTestObject* Object = NewObject<UAutoRTFMTestObject>();
		FTimerHandle TimerHandle = TimerManager.SetTimerForNextTick(Object, &UAutoRTFMTestObject::SetValueTo100);

		AutoRTFM::Testing::Abort([&]
		{
			TimerManager.ClearAllTimersForObject(Object);
			check(!TimerManager.IsTimerActive(TimerHandle));
			AutoRTFM::AbortTransaction();
		});

		check(TimerManager.IsTimerActive(TimerHandle));
		Tick(TimerManager);

		check(Object->Value == 100);

		return true;
	});

	Section(TEXT("Transact(SetTimerForNextTick, ClearAllTimersForObject), Tick"), [&] (FTimerManager& TimerManager)
	{
		UAutoRTFMTestObject* Object = NewObject<UAutoRTFMTestObject>();

		AutoRTFM::Testing::Commit([&]
		{
			FTimerHandle TimerHandle = TimerManager.SetTimerForNextTick(Object, &UAutoRTFMTestObject::SetValueTo100);

			TimerManager.ClearAllTimersForObject(Object);
		});

		Tick(TimerManager);

		check(Object->Value != 100);

		return true;
	});

	Section(TEXT("Transact(SetTimerForNextTick, ClearAllTimersForObject, Abort), Tick"), [&] (FTimerManager& TimerManager)
	{
		UAutoRTFMTestObject* Object = NewObject<UAutoRTFMTestObject>();

		AutoRTFM::Testing::Abort([&]
		{
			FTimerHandle TimerHandle = TimerManager.SetTimerForNextTick(Object, &UAutoRTFMTestObject::SetValueTo100);

			TimerManager.ClearAllTimersForObject(Object);

			AutoRTFM::AbortTransaction();
		});

		Tick(TimerManager);

		check(Object->Value != 100);

		return true;
	});

	Section(TEXT("Transact(SetTimer, ClearTimer, ClearTimer, Abort)"), [&] (FTimerManager& TimerManager)
	{
		FTimer Timer{};

		AutoRTFM::Testing::Abort([&]
		{
			TimerManager.SetTimer(Timer.Handle, Timer.Callback(), /* InRate */ 1.0f, FTimerManagerTimerParameters{});

			ClearTimer(TimerManager, Timer.Handle);

			ClearTimer(TimerManager, Timer.Handle);
			
			AutoRTFM::AbortTransaction();
		});

		return true;
	});

	Section(TEXT("SetTimerForNextTick, ClearTimer, Tick, SetTimerForNextTick, Transact(ClearTimer, Abort)"), [&] (FTimerManager& TimerManager)
	{
		FTimerHandle HandleA = TimerManager.SetTimerForNextTick([]{});

		ClearTimer(TimerManager, HandleA);

		Tick(TimerManager);

		// As the timer with HandleA was cleared and then ticked, the FTimerHandle.Timers slot
		// that contained it should now be vacant. Creating a new timer should reuse that slot.

		FTimerHandle HandleB = TimerManager.SetTimerForNextTick([]{});

		// HandleA and HandleB now share the same slot index, but have different serials.
		AutoRTFM::Testing::Abort([&]
		{
			// Attempting to reuse the dead handle should be a no-op.
			TimerManager.SetTimer(HandleA, []{}, /* InRate */ 1.0f, FTimerManagerTimerParameters{});
			
			AutoRTFM::AbortTransaction();
		});

		return true;
	});

	Section(TEXT("Transact(SetTimer, ClearTimer, PauseTimer, Abort)"), [&] (FTimerManager& TimerManager)
	{
		FTimer Timer{};

		AutoRTFM::Testing::Abort([&]
		{
			TimerManager.SetTimer(Timer.Handle, Timer.Callback(), /* InRate */ 1.0f, FTimerManagerTimerParameters{});

			ClearTimer(TimerManager, Timer.Handle);

			TimerManager.PauseTimer(Timer.Handle);
			
			AutoRTFM::AbortTransaction();
		});

		return true;
	});

	Section(TEXT("Transact(SetTimerForNextTick, ClearTimer, ClearTimer, Abort)"), [&] (FTimerManager& TimerManager)
	{
		FTimer Timer{};

		AutoRTFM::Testing::Abort([&]
		{
			Timer.Handle = TimerManager.SetTimerForNextTick(Timer.Callback());

			ClearTimer(TimerManager, Timer.Handle);

			ClearTimer(TimerManager, Timer.Handle);
			
			AutoRTFM::AbortTransaction();
		});

		return true;
	});

	Section(TEXT("Transact(SetTimerForNextTick, ClearTimer, PauseTimer, Abort)"), [&] (FTimerManager& TimerManager)
	{
		FTimer Timer{};

		AutoRTFM::Testing::Abort([&]
		{
			Timer.Handle = TimerManager.SetTimerForNextTick(Timer.Callback());

			ClearTimer(TimerManager, Timer.Handle);

			TimerManager.PauseTimer(Timer.Handle);
			
			AutoRTFM::AbortTransaction();
		});

		return true;
	});

	Section(TEXT("SetTimerForNextTick(Transact(ClearTimer, ClearTimer)))"), [&] (FTimerManager& TimerManager)
	{
		FTimerHandle TimerHandle;
		TimerHandle = TimerManager.SetTimerForNextTick([&]
		{
			AutoRTFM::Testing::Commit([&]
			{
				ClearTimer(TimerManager, TimerHandle);

				ClearTimer(TimerManager, TimerHandle);
			});
		});

		Tick(TimerManager);

		return true;
	});

	Section(TEXT("SetTimer(Early) Transact(SetTimer(Late))"), [&] (FTimerManager& TimerManager)
	{
		TimerManager.Tick(1000.0f);

		FTimer EarlyTimer{};
		TimerManager.SetTimer(EarlyTimer.Handle, EarlyTimer.Callback(), /* InRate */ 1.0f, FTimerManagerTimerParameters{});
		// EarlyTimer.ExpireTime = InternalTime(1000.0f) + InRate(1.0)

		FTimer LateTimer{};
		AutoRTFM::Testing::Commit([&]
		{
			TimerManager.SetTimer(LateTimer.Handle, LateTimer.Callback(), /* InRate */ 1000.0f, FTimerManagerTimerParameters{});
			// LateTimer.ExpireTime = InternalTime(1000.0f) + InRate(1000.0)
		});

		// LateTimer has a much larger ExpireTime, but was inserted with a InRate less than EarlyTimer.ExpireTime.
		// If ExpireTime was not increased by InternalTime before the call to ActiveTimerHeap.HeapPush()
		// then LateTimer will block EarlyTimer from running.
		Tick(TimerManager);

		check(EarlyTimer.bCalled);
		check(!LateTimer.bCalled);

		return true;
	});

	// The following tests assert that SetTimerForNextTick called from inside a closed
	// AutoRTFM transaction fires on the very next FTimerManager::Tick() - matching the
	// open-path behavior when GuaranteeEngineTickDelay == 0 (the default).
	auto SingleTick = [&](FTimerManager& TimerManager)
	{
		GFrameCounter++;
		TimerManager.Tick(1.5f);
	};

	Section(TEXT("Transact(SetTimerForNextTick), SingleTick"), [&] (FTimerManager& TimerManager)
	{
		constexpr size_t NumTimers = 16;
		FTimer Timers[NumTimers]{};

		AutoRTFM::Testing::Commit([&]
		{
			for (FTimer& Timer : Timers)
			{
				Timer.Handle = TimerManager.SetTimerForNextTick(Timer.Callback());
			}
		});

		// One tick is enough - the closed-scheduled timers must not be deferred to a
		// second frame.
		SingleTick(TimerManager);

		for (FTimer const& Timer : Timers)
		{
			check(Timer.bCalled);
		}

		return true;
	});

	Section(TEXT("Open(SetTimerForNextTick), Transact(SetTimerForNextTick), SingleTick"), [&] (FTimerManager& TimerManager)
	{
		// The open path and the closed path must agree on same-frame execution: a
		// single Tick after both registrations must fire both timers.
		FTimer OpenTimer{};
		FTimer ClosedTimer{};

		OpenTimer.Handle = TimerManager.SetTimerForNextTick(OpenTimer.Callback());

		AutoRTFM::Testing::Commit([&]
		{
			ClosedTimer.Handle = TimerManager.SetTimerForNextTick(ClosedTimer.Callback());
		});

		SingleTick(TimerManager);

		check(OpenTimer.bCalled);
		check(ClosedTimer.bCalled);

		return true;
	});

	Section(TEXT("Transact(Open(SetTimerForNextTick), SetTimerForNextTick), SingleTick"), [&] (FTimerManager& TimerManager)
	{
		// Open and closed SetTimerForNextTick calls inside the same transaction must
		// both fire on a single subsequent tick.
		FTimer OpenTimer{};
		FTimer ClosedTimer{};

		AutoRTFM::Testing::Commit([&]
		{
			AutoRTFM::Open([&]
			{
				OpenTimer.Handle = TimerManager.SetTimerForNextTick(OpenTimer.Callback());
			});

			ClosedTimer.Handle = TimerManager.SetTimerForNextTick(ClosedTimer.Callback());
		});

		SingleTick(TimerManager);

		check(OpenTimer.bCalled);
		check(ClosedTimer.bCalled);

		return true;
	});

	Section(TEXT("Transact(SetTimerForNextTick) x N, SingleTick"), [&] (FTimerManager& TimerManager)
	{
		// Mirrors the camera repro: many short transactions in succession, each
		// scheduling a next-tick timer, then a single Tick. All must fire.
		constexpr size_t NumTransactions = 8;
		constexpr size_t NumTimersPerTransaction = 4;
		FTimer Timers[NumTransactions][NumTimersPerTransaction]{};

		for (size_t T = 0; T < NumTransactions; T++)
		{
			AutoRTFM::Testing::Commit([&]
			{
				for (FTimer& Timer : Timers[T])
				{
					Timer.Handle = TimerManager.SetTimerForNextTick(Timer.Callback());
				}
			});
		}

		SingleTick(TimerManager);

		for (size_t T = 0; T < NumTransactions; T++)
		{
			for (FTimer const& Timer : Timers[T])
			{
				check(Timer.bCalled);
			}
		}

		return true;
	});

	Section(TEXT("Transact(ClearTimer(invalid-handle))"), [&] (FTimerManager& TimerManager)
	{
		AutoRTFM::Testing::Commit([&]
		{
			FTimerHandle InvalidHandle;
			TimerManager.ClearTimer(InvalidHandle);
		});

		return true;
	});

	Section(TEXT("Transact(ClearTimer(invalid-handle), Abort)"), [&] (FTimerManager& TimerManager)
	{
		AutoRTFM::Testing::Abort([&]
		{
			FTimerHandle InvalidHandle;
			TimerManager.ClearTimer(InvalidHandle);
			AutoRTFM::AbortTransaction();
		});

		return true;
	});

	Section(TEXT("Transact(PauseTimer(invalid-handle))"), [&] (FTimerManager& TimerManager)
	{
		AutoRTFM::Testing::Commit([&]
		{
			FTimerHandle InvalidHandle;
			TimerManager.PauseTimer(InvalidHandle);
		});

		return true;
	});

	Section(TEXT("Transact(PauseTimer(invalid-handle), Abort)"), [&] (FTimerManager& TimerManager)
	{
		AutoRTFM::Testing::Abort([&]
		{
			FTimerHandle InvalidHandle;
			TimerManager.PauseTimer(InvalidHandle);
			AutoRTFM::AbortTransaction();
		});

		return true;
	});

	Section(TEXT("Transact(UnpauseTimer(invalid-handle))"), [&] (FTimerManager& TimerManager)
	{
		AutoRTFM::Testing::Commit([&]
		{
			FTimerHandle InvalidHandle;
			TimerManager.UnPauseTimer(InvalidHandle);
		});

		return true;
	});
	Section(TEXT("Transact(UnpauseTimer(invalid-handle), Abort)"), [&] (FTimerManager& TimerManager)
	{
		AutoRTFM::Testing::Abort([&]
		{
			FTimerHandle InvalidHandle;
			TimerManager.UnPauseTimer(InvalidHandle);
			AutoRTFM::AbortTransaction();
		});

		return true;
	});

	return true;
}

#endif //WITH_DEV_AUTOMATION_TESTS
