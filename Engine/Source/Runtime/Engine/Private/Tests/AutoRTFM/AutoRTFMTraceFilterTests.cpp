// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/AutomationTest.h"
#include "TraceFilter.h"
#include "AutoRTFM.h"
#include "AutoRTFMTestObject.h"

#if WITH_DEV_AUTOMATION_TESTS

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FAutoRTFMTraceFilterTests, "AutoRTFM + FTraceFilter", EAutomationTestFlags::EngineFilter | EAutomationTestFlags::ClientContext | EAutomationTestFlags::ServerContext | EAutomationTestFlags::CommandletContext)

bool FAutoRTFMTraceFilterTests::RunTest(const FString & Parameters)
{
	if (!AutoRTFM::ForTheRuntime::IsAutoRTFMRuntimeEnabled())
	{
		ExecutionInfo.AddEvent(FAutomationEvent(EAutomationEventType::Info, TEXT("SKIPPED 'FAutoRTFMTraceFilterTests' test. AutoRTFM disabled.")));
		return true;
	}

#if TRACE_FILTERING_ENABLED
	{
		UAutoRTFMTestObject* Object = NewObject<UAutoRTFMTestObject>();

		FTraceFilter::SetObjectIsTraceable(Object, false);
		TestFalseExpr(FTraceFilter::IsObjectTraceable(Object));

		AutoRTFM::ETransactionResult Result = AutoRTFM::Transact([&]
			{
				FTraceFilter::SetObjectIsTraceable(Object, true);
				AutoRTFM::AbortTransaction();
			});

		TestTrueExpr(AutoRTFM::ETransactionResult::AbortedByRequest == Result);
		TestFalseExpr(FTraceFilter::IsObjectTraceable(Object));

		Result = AutoRTFM::Transact([&]
			{
				FTraceFilter::SetObjectIsTraceable(Object, true);
			});

		TestTrueExpr(AutoRTFM::ETransactionResult::Committed == Result);
		TestTrueExpr(FTraceFilter::IsObjectTraceable(Object));

		Result = AutoRTFM::Transact([&]
			{
				AutoRTFM::OnAbort([Object]
					{
						FTraceFilter::SetObjectIsTraceable(Object, false);
					});

				AutoRTFM::AbortTransaction();
			});

		TestFalseExpr(FTraceFilter::IsObjectTraceable(Object));

		Result = AutoRTFM::Transact([&]
			{
				AutoRTFM::OnCommit([Object]
					{
						FTraceFilter::SetObjectIsTraceable(Object, true);
					});
			});

		TestTrueExpr(FTraceFilter::IsObjectTraceable(Object));

		FTraceFilter::SetObjectIsTraceable(Object, false);
		TestFalseExpr(FTraceFilter::IsObjectTraceable(Object));

		UAutoRTFMTestObject* Other = NewObject<UAutoRTFMTestObject>();
		UAutoRTFMTestObject* Another = NewObject<UAutoRTFMTestObject>();

		FTraceFilter::SetObjectIsTraceable(Other, false);
		TestFalseExpr(FTraceFilter::IsObjectTraceable(Other));

		Result = AutoRTFM::Transact([&]
			{
				AutoRTFM::OnAbort([&]
					{
						FTraceFilter::SetObjectIsTraceable(Other, true);
					});

				FTraceFilter::SetObjectIsTraceable(Object, true);

				AutoRTFM::OnAbort([&]
					{
						FTraceFilter::SetObjectIsTraceable(Another, true);
					});

				AutoRTFM::AbortTransaction();
			});

		TestTrueExpr(AutoRTFM::ETransactionResult::AbortedByRequest == Result);
		TestFalseExpr(FTraceFilter::IsObjectTraceable(Object));
		TestTrueExpr(FTraceFilter::IsObjectTraceable(Other));
		TestTrueExpr(FTraceFilter::IsObjectTraceable(Another));

		Result = AutoRTFM::Transact([&]
			{
				AutoRTFM::OnCommit([&]
					{
						FTraceFilter::SetObjectIsTraceable(Other, false);
					});

				FTraceFilter::SetObjectIsTraceable(Object, true);

				AutoRTFM::OnCommit([&]
					{
						FTraceFilter::SetObjectIsTraceable(Another, false);
					});
			});

		TestTrueExpr(AutoRTFM::ETransactionResult::Committed == Result);
		TestTrueExpr(FTraceFilter::IsObjectTraceable(Object));
		TestFalseExpr(FTraceFilter::IsObjectTraceable(Other));
		TestFalseExpr(FTraceFilter::IsObjectTraceable(Another));
	}

	{
		UAutoRTFMTestObject* Object = NewObject<UAutoRTFMTestObject>();

		FTraceFilter::SetObjectIsTraceable(Object, false);
		TestFalseExpr(FTraceFilter::IsObjectTraceable(Object));

		FTraceFilter::MarkObjectTraceable(Object);
		TestTrueExpr(FTraceFilter::IsObjectTraceable(Object));

		FTraceFilter::SetObjectIsTraceable(Object, false);
		TestFalseExpr(FTraceFilter::IsObjectTraceable(Object));

		AutoRTFM::ETransactionResult Result = AutoRTFM::Transact([&]
			{
				FTraceFilter::MarkObjectTraceable(Object);
				AutoRTFM::AbortTransaction();
			});

		TestTrueExpr(AutoRTFM::ETransactionResult::AbortedByRequest == Result);
		TestFalseExpr(FTraceFilter::IsObjectTraceable(Object));

		Result = AutoRTFM::Transact([&]
			{
				FTraceFilter::MarkObjectTraceable(Object);
			});

		TestTrueExpr(AutoRTFM::ETransactionResult::Committed == Result);
		TestTrueExpr(FTraceFilter::IsObjectTraceable(Object));

		FTraceFilter::SetObjectIsTraceable(Object, false);
		TestFalseExpr(FTraceFilter::IsObjectTraceable(Object));

		Result = AutoRTFM::Transact([&]
			{
				AutoRTFM::OnAbort([Object]
					{
						FTraceFilter::MarkObjectTraceable(Object);
					});

				AutoRTFM::AbortTransaction();
			});

		TestTrueExpr(FTraceFilter::IsObjectTraceable(Object));

		FTraceFilter::SetObjectIsTraceable(Object, false);
		TestFalseExpr(FTraceFilter::IsObjectTraceable(Object));

		Result = AutoRTFM::Transact([&]
			{
				AutoRTFM::OnCommit([Object]
					{
						FTraceFilter::MarkObjectTraceable(Object);
					});
			});

		TestTrueExpr(FTraceFilter::IsObjectTraceable(Object));
	}

	{
		UAutoRTFMTestObject* Object = NewObject<UAutoRTFMTestObject>();

		bool bTraceable = true;

		AutoRTFM::ETransactionResult Result = AutoRTFM::Transact([&]
			{
				bTraceable = FTraceFilter::IsObjectTraceable(Object);
			});

		TestTrueExpr(AutoRTFM::ETransactionResult::Committed == Result);
		TestTrueExpr(FTraceFilter::IsObjectTraceable(Object) == bTraceable);
	}

	return true;
#else
	ExecutionInfo.AddEvent(FAutomationEvent(EAutomationEventType::Info, TEXT("SKIPPED 'FAutoRTFMTraceFilterTests' test. Trace filtering disabled.")));
	return true;
#endif
}

#endif //WITH_DEV_AUTOMATION_TESTS
