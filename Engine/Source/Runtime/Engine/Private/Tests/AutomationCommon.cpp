// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tests/AutomationCommon.h"
#include "AssetCompilingManager.h"
#include "Engine/World.h"
#include "Misc/EngineVersion.h"
#include "Engine/GameViewportClient.h"
#include "Engine/GameInstance.h"
#include "Engine/Engine.h"
#include "HAL/PlatformFile.h"
#include "HardwareInfo.h"
#include "Misc/App.h"
#include "UObject/Package.h"
#include "Kismet/GameplayStatics.h"
#include "ContentStreaming.h"
#include "Framework/Application/SlateApplication.h"
#include "RenderingThread.h"
#include "ShaderCompiler.h"
#include "GameFramework/GameStateBase.h"
#include "Scalability.h"
#include "StereoRendering.h"
#include "Misc/PackageName.h"
#include "Tests/AutomationTestSettings.h"
#include "GameMapsSettings.h"
#include "IRenderCaptureProvider.h"
#include "Modules/ModuleManager.h"
#include "Algo/Accumulate.h"
#include "RenderUtils.h"

#if WITH_AUTOMATION_TESTS

DEFINE_LOG_CATEGORY_STATIC(LogEngineAutomationLatentCommand, Log, All);
DEFINE_LOG_CATEGORY(LogEditorAutomationTests);
DEFINE_LOG_CATEGORY(LogEngineAutomationTests);

static TAutoConsoleVariable<int32> CVarAutomationAllowFrameTraceCapture(
	TEXT("AutomationAllowFrameTraceCapture"),
	1,
	TEXT("Allow automation to capture frame traces."),
	ECVF_Default
	);

//declare static variable
FOnEditorAutomationMapLoad AutomationCommon::OnEditorAutomationMapLoad;
#endif

///////////////////////////////////////////////////////////////////////
// FTestWorldWrapper

FTestWorldWrapper::~FTestWorldWrapper()
{
	// Make sure to destroy the test world unless this is a hard exit
	if (UObjectInitialized() && !IsEngineExitRequested())
	{
		DestroyTestWorld(false);
	}
}

bool FTestWorldWrapper::CreateTestWorld(EWorldType::Type WorldType)
{
	if (TestWorld != nullptr)
	{
		ReportFailure(TEXT("TestWorld already exists in CreateTestWorld!"));
		return false;
	}

	UGameInstance* TestGameInstance = nullptr;

	switch (WorldType)
	{
		case EWorldType::Game:
			// Create something close to a game world that can be manually ticked
			TestGameInstance = NewObject<UGameInstance>(GEngine);
			break;

		case EWorldType::Editor:
		case EWorldType::EditorPreview:
		case EWorldType::GamePreview:
		case EWorldType::GameRPC:
		case EWorldType::Inactive:
			// Don't initialize game instance
			break;

		case EWorldType::PIE: // TODO Implement if needed
		default:
			ensureMsgf(false, TEXT("Unsupported world type in CreateTestWorld"));
			ReportFailure(TEXT("Unsupported world type in CreateTestWorld"));
			return false;
	}

	TestWorld = UWorld::CreateWorld(WorldType, false);

	if (!TestWorld)
	{
		ReportFailure(TEXT("Failed to create world in CreateTestWorld"));
		return false;
	}

	// Will be ticked manually
	TestWorld->SetShouldTick(false);
	TestWorld->AddToRoot();
	
	FWorldContext& WorldContext = GEngine->CreateNewWorldContext(WorldType);
	WorldContext.OwningGameInstance = TestGameInstance;

	TestWorld->SetGameInstance(TestGameInstance);
	WorldContext.SetCurrentWorld(TestWorld);

	if (TestGameInstance)
	{
		TestGameInstance->Init();
	}
	
	return true;
}

bool FTestWorldWrapper::DestroyTestWorld(bool bForceGarbageCollect)
{
	if (TestWorld == nullptr)
	{
		ReportFailure(TEXT("TestWorld does not exist in DestroyTestWorld!"));
		return false;
	}

	// End play if it hasn't already happened, this will do nothing if it didn't spawn a game mode
	EndPlayInTestWorld();

	TestWorld->RemoveFromRoot();
	if (TestWorld->GetGameInstance() != nullptr)
	{
		TestWorld->GetGameInstance()->Shutdown();
	}

	GEngine->DestroyWorldContext(TestWorld);
	TestWorld->DestroyWorld(false);
	TestWorld = nullptr;

	if (bForceGarbageCollect)
	{
		CollectGarbage(GARBAGE_COLLECTION_KEEPFLAGS);

		// Verify it was actually destroyed
		GEngine->CheckAndHandleStaleWorldObjectReferences();
	}

	return true;
}

bool FTestWorldWrapper::BeginPlayInTestWorld()
{
	if (TestWorld == nullptr)
	{
		ReportFailure(TEXT("TestWorld does not exist in BeginPlayInTestWorld!"));
		return false;
	}

	if (TestWorld->HasBegunPlay())
	{
		ReportFailure(TEXT("TestWorld has already begin play in BeginPlayInTestWorld!"));
		return false;
	}

	CachedFrameCounter = GFrameCounter;

	FURL URL;
	if (TestWorld->GetGameInstance())
	{
		// This is required to actually forward actor BeginPlay
		TestWorld->SetGameMode(URL);

		if (!TestWorld->GetAuthGameMode())
		{
			ReportFailure(TEXT("BeginPlayInTestWorld failed to create GameMode"));
			return false;
		}

		if (!TestWorld->GetWorldSettings())
		{
			ReportFailure(TEXT("BeginPlayInTestWorld failed to create WorldSettings"));
			return false;
		}
	}

	TestWorld->InitializeActorsForPlay(URL);
	TestWorld->BeginPlay();

	if (TestWorld->GetGameInstance())
	{
		if (!TestWorld->GetGameState())
		{
			ReportFailure(TEXT("BeginPlayInTestWorld failed to create GameState"));
			return false;
		}
	}

	return true;
}

bool FTestWorldWrapper::EndPlayInTestWorld()
{
	if (!TestWorld)
	{
		ReportFailure(TEXT("Failed to find world in EndPlayInTestWorld"));
		return false;
	}

	// Ignore if this never started play
	if (TestWorld->HasBegunPlay())
	{
		TestWorld->BeginTearingDown();
		TestWorld->EndPlay(EEndPlayReason::Quit);

		// Restore the frame counter
		GFrameCounter = CachedFrameCounter;
	}

	return true;
}

bool FTestWorldWrapper::TickTestWorld(float DeltaTime)
{
	if (!TestWorld)
	{
		ReportFailure(TEXT("Failed to find world in TickTestWorld"));
		return false;
	}

	TestWorld->Tick(LEVELTICK_All, DeltaTime);
	if (TestWorld->HasBegunPlay())
	{
		// If this has begin play increment the global counter to emulate gameplay
		GFrameCounter++;
	}

	return true;
}

void FTestWorldWrapper::ReportFailure(const TCHAR* ErrorMessage)
{
	FailureErrors.Emplace(ErrorMessage);
}

void FTestWorldWrapper::ClearFailureState()
{
	FailureErrors.Reset();
}

bool FTestWorldWrapper::HasFailed() const
{
	return FailureErrors.Num() > 0;
}

void FTestWorldWrapper::AppendErrorMessages(TArray<FString>& OutErrorMessages) const
{
	OutErrorMessages.Append(FailureErrors);
}

void FTestWorldWrapper::ForwardErrorMessages(FAutomationTestBase* AutomationTest) const
{
	if (HasFailed())
	{
		TArray<FString> FailureMessages;
		AppendErrorMessages(FailureMessages);
		for (const FString& Failure : FailureMessages)
		{
			AutomationTest->AddError(Failure);
		}
	}
}

///////////////////////////////////////////////////////////////////////
// FTestConsoleVariable

FTestConsoleVariable::FTestConsoleVariable(const FString& InConsoleVariableName)
	: bModified(false)
	, ConsoleVariableName(InConsoleVariableName)
{
}

FTestConsoleVariable::FTestConsoleVariable(FTestConsoleVariable&& Other)
	: bModified(MoveTemp(Other.bModified))
	, ConsoleVariableName(MoveTemp(Other.ConsoleVariableName))
	, OriginalValue(MoveTemp(Other.OriginalValue))
{
	// Reset our other's modified flag to avoid triggering a reset of the CVar
	Other.bModified = false;
}

FTestConsoleVariable::~FTestConsoleVariable()
{
	Restore();
}

void FTestConsoleVariable::Set(const FString& Value)
{
	IConsoleVariable* ConsoleVariable = IConsoleManager::Get().FindConsoleVariable(*ConsoleVariableName);
	if (ensure(ConsoleVariable))
	{
		if (!bModified)
		{
			bModified = true;
			OriginalValue = ConsoleVariable->GetString();
		}

		ConsoleVariable->AsVariable()->SetWithCurrentPriority(*Value);

#if WITH_AUTOMATION_TESTS
		UE_LOGF(LogEngineAutomationTests, Verbose, "CVar '%ls' was set with the value '%ls'.", *ConsoleVariableName, *Value);
#endif // WITH_AUTOMATION_TESTS
	}
}

FString FTestConsoleVariable::Get()
{
	IConsoleVariable* ConsoleVariable = IConsoleManager::Get().FindConsoleVariable(*ConsoleVariableName);
	if (ensure(ConsoleVariable))
	{
		return ConsoleVariable->GetString();
	}

	return FString{};
}

void FTestConsoleVariable::Restore()
{
	if (bModified)
	{
		IConsoleVariable* ConsoleVariable = IConsoleManager::Get().FindConsoleVariable(*ConsoleVariableName);
		if (ensure(ConsoleVariable))
		{
			ConsoleVariable->AsVariable()->SetWithCurrentPriority(*OriginalValue);

#if WITH_AUTOMATION_TESTS
			UE_LOGF(LogEngineAutomationTests, Verbose, "CVar '%ls' was restored with the original value '%ls'.", *ConsoleVariableName, *OriginalValue);
#endif // WITH_AUTOMATION_TESTS
		}

		bModified = false;
	}
}

///////////////////////////////////////////////////////////////////////
// FScopedTestEnvironment

TWeakPtr<FScopedTestEnvironment> FScopedTestEnvironment::EnvironmentInstance = nullptr;

FScopedTestEnvironment::~FScopedTestEnvironment()
{
	Restore();
}

TSharedPtr<FScopedTestEnvironment> FScopedTestEnvironment::Get()
{
	check(IsInGameThread());

	TSharedPtr<FScopedTestEnvironment> Instance = EnvironmentInstance.Pin();
	if (!Instance.IsValid())
	{
		Instance = MakeShareable(new FScopedTestEnvironment());
		EnvironmentInstance = Instance;
	}

	check(Instance.IsValid());
	return Instance;
}

void FScopedTestEnvironment::SetConsoleVariableValue(const FString& ConsoleVariableName, const FString& Value)
{
	check(IsInGameThread());

	if (FTestConsoleVariable* Variable = Variables.Find(ConsoleVariableName))
	{
		Variable->Set(Value);
	}
	else
	{
		FTestConsoleVariable NewVariable(ConsoleVariableName);
		NewVariable.Set(Value);
		Variables.Emplace(ConsoleVariableName, MoveTemp(NewVariable));
	}
}

bool FScopedTestEnvironment::TryGetConsoleVariableValue(const FString& ConsoleVariableName, FString* OutValue)
{
	check(IsInGameThread());

	if (FTestConsoleVariable* Variable = Variables.Find(ConsoleVariableName))
	{
		if (OutValue)
		{
			*OutValue = Variable->Get();
		}
		return true;
	}

	return false;
}

void FScopedTestEnvironment::Restore()
{
	if (!Variables.IsEmpty())
	{
		check(IsInGameThread());

		Variables.Empty();
	}
}

///////////////////////////////////////////////////////////////////////
// Common Latent commands

namespace AutomationCommon
{
#if WITH_AUTOMATION_TESTS
	FString GetRenderDetailsString()
	{
		FString HardwareDetailsString;

		// Create the folder name based on the hardware specs we have been provided
		FString HardwareDetails = FHardwareInfo::GetHardwareDetailsString();

		FString RHIString;
		FString RHILookup = NAME_RHI.ToString() + TEXT("=");
		if ( FParse::Value(*HardwareDetails, *RHILookup, RHIString) )
		{
			HardwareDetailsString = ( HardwareDetailsString + TEXT("_") ) + RHIString;
		}

		FString TextureFormatString;
		FString TextureFormatLookup = NAME_TextureFormat.ToString() + TEXT("=");
		if ( FParse::Value(*HardwareDetails, *TextureFormatLookup, TextureFormatString) )
		{
			HardwareDetailsString = ( HardwareDetailsString + TEXT("_") ) + TextureFormatString;
		}

		FString DeviceTypeString;
		FString DeviceTypeLookup = NAME_DeviceType.ToString() + TEXT("=");
		if ( FParse::Value(*HardwareDetails, *DeviceTypeLookup, DeviceTypeString) )
		{
			HardwareDetailsString = ( HardwareDetailsString + TEXT("_") ) + DeviceTypeString;
		}

		FString FeatureLevelString;
		GetFeatureLevelName(GMaxRHIFeatureLevel, FeatureLevelString);
		{
			HardwareDetailsString = ( HardwareDetailsString + TEXT("_") ) + FeatureLevelString;
		}

		if ( HardwareDetailsString.Len() > 0 )
		{
			//Get rid of the leading "_"
			HardwareDetailsString.RightChopInline(1, EAllowShrinking::No);
		}

		return HardwareDetailsString;
	}

	/** Gets a path used for automation testing (PNG sent to the AutomationTest folder) */
	FString GetScreenshotPath(const FString& TestName)
	{
		FString PathName = TestName / FPlatformProperties::IniPlatformName();
		PathName = PathName + TEXT("/") + GetRenderDetailsString();

		// We need a unique ID for filenames from this run. We used to use GetDeviceId() but that is not guaranteed to return
		// a valid string on some platforms.
		static FString UUID = FGuid::NewGuid().ToString(EGuidFormats::Short).ToLower();

		return FString::Printf(TEXT("%s/%s.png"), *PathName, *UUID);
	}

	FString GetLocalPathForScreenshot(const FString& InScreenshotName)
	{
		return FPaths::AutomationDir() + InScreenshotName;
	}
	
	FAutomationScreenshotData BuildScreenshotData(const FString& MapOrContext, const FString& TestName, const FString& ScreenShotName, int32 Width, int32 Height)
	{
		FAutomationScreenshotData Data;

		Data.ScreenShotName = FPaths::MakeValidFileName(ScreenShotName, TEXT('_'));
		Data.Context = MapOrContext;
		Data.TestName = TestName;
		Data.Id = FGuid::NewGuid();
		Data.Commit = FEngineVersion::Current().HasChangelist() ? FString::FromInt(FEngineVersion::Current().GetChangelist()) : FString();

		Data.Width = Width;
		Data.Height = Height;
		Data.Platform = FPlatformProperties::IniPlatformName();
		Data.Rhi = FHardwareInfo::GetHardwareInfo(NAME_RHI);
		GetFeatureLevelName(GMaxRHIFeatureLevel, Data.FeatureLevel);
		Data.bIsStereo = GEngine->StereoRenderingDevice.IsValid() ? GEngine->StereoRenderingDevice->IsStereoEnabled() : false;
		Data.bIsSubstrate = Substrate::IsSubstrateEnabled();
		Data.Vendor = RHIVendorIdToString();
		Data.AdapterName = GRHIAdapterName;
		Data.AdapterInternalDriverVersion = GRHIAdapterInternalDriverVersion;
		Data.AdapterUserDriverVersion = GRHIAdapterUserDriverVersion;
		Data.UniqueDeviceId = FPlatformMisc::GetDeviceId();

		Scalability::FQualityLevels QualityLevels = Scalability::GetQualityLevels();

		Data.ResolutionQuality = QualityLevels.ResolutionQuality;
		Data.ViewDistanceQuality = QualityLevels.ViewDistanceQuality;
		Data.AntiAliasingQuality = QualityLevels.AntiAliasingQuality;
		Data.ShadowQuality = QualityLevels.ShadowQuality;
		Data.GlobalIlluminationQuality = QualityLevels.GlobalIlluminationQuality;
		Data.ReflectionQuality = QualityLevels.ReflectionQuality;
		Data.PostProcessQuality = QualityLevels.PostProcessQuality;
		Data.TextureQuality = QualityLevels.TextureQuality;
		Data.EffectsQuality = QualityLevels.EffectsQuality;
		Data.FoliageQuality = QualityLevels.FoliageQuality;
		Data.ShadingQuality = QualityLevels.ShadingQuality;
		
		//GRHIDeviceId

		// TBD - 
		// Device's native resolution (we want to use a hardware dump of the frontbuffer at the native resolution so we compare what we actually output rather than what we think we rendered)

		const FString MapAndTest = MapOrContext + TEXT("/") + Data.ScreenShotName;
		Data.ScreenshotPath = GetScreenshotPath(MapAndTest);

		return Data;
	}

	TArray<uint8> CaptureFrameTrace(const FString& MapOrContext, const FString& TestName)
	{
		TArray<uint8> FrameTrace;

		if (CVarAutomationAllowFrameTraceCapture.GetValueOnGameThread() != 0
			&& IRenderCaptureProvider::IsAvailable()
			&& FModuleManager::Get().IsModuleLoaded("RenderDocPlugin"))
		{
			const FString MapAndTest = MapOrContext / FPaths::MakeValidFileName(TestName, TEXT('_'));
			FString ScreenshotPath = GetScreenshotPath(MapAndTest);
			FString TempCaptureFilePath = FPaths::ChangeExtension(FPaths::ConvertRelativePathToFull(FPaths::AutomationDir() / TEXT("Incoming/") / ScreenshotPath), TEXT(".rdc"));

			UE_LOGF(LogEngineAutomationTests, Log, "Taking Frame Trace: %ls", *TempCaptureFilePath);

			IRenderCaptureProvider::Get().CaptureFrame(GEngine->GameViewport->Viewport, 0, TempCaptureFilePath);
			FlushRenderingCommands();

			IPlatformFile& PlatformFileSystem = IPlatformFile::GetPlatformPhysical();
			if (PlatformFileSystem.FileExists(*TempCaptureFilePath))
			{
				{
					TUniquePtr<IFileHandle> FileHandle(PlatformFileSystem.OpenRead(*TempCaptureFilePath));

					int64 FileSize = FileHandle->Size();
					FrameTrace.SetNumUninitialized(FileSize);
					FileHandle->Read(FrameTrace.GetData(), FileSize);
				}

				PlatformFileSystem.DeleteFile(*TempCaptureFilePath);
			}
			else
			{
				UE_LOGF(LogEngineAutomationTests, Warning, "Failed taking frame trace: %ls", *TempCaptureFilePath);
			}
		}

		return FrameTrace;
	}

	SWidget* FindWidgetByTag(const FName Tag)
	{
		const FTagMetaData UniqueMetaData(Tag);
		// Get a list of all the current slate windows
		TArray<TSharedRef<SWindow>> Windows;
		FSlateApplication::Get().GetAllVisibleWindowsOrdered(/*OUT*/Windows);

		TArray<SWidget*> Stack;
		for (const TSharedRef<SWindow>& Window : Windows)
		{
			Stack.Push(&Window.Get());
		}

		while (Stack.Num() > 0)
		{
			SWidget* Widget = Stack.Pop();
			const int32 NumChildren = Widget->GetChildren()->Num();
			for (int32 ChildIndex = 0; ChildIndex < NumChildren; ChildIndex++)
			{
				SWidget& ChildWidget = Widget->GetChildren()->GetChildAt(ChildIndex).Get();
				const TArray<TSharedRef<FTagMetaData>> AllMetaData = ChildWidget.GetAllMetaData<FTagMetaData>();
				for (int32 MetaDataIndex = 0; MetaDataIndex < AllMetaData.Num(); ++MetaDataIndex)
				{
					TSharedRef<FTagMetaData> MetaData = AllMetaData[MetaDataIndex];
					if (MetaData->Tag == UniqueMetaData.Tag)
					{
						// Done! found the widget
						return &ChildWidget;
					}
				}

				// If we got here we didn't match the widget so push this child on the stack.
				Stack.Push(&ChildWidget);
			}

		}

		return nullptr;
	}

	class FAutomationImageComparisonRequest : public IAutomationLatentCommand
	{
	public:
		FAutomationImageComparisonRequest(const FString& InImageName, const FString& InContext, int32 InWidth, int32 InHeight, const TArray<FColor>& InImageData, const FAutomationComparisonToleranceAmount& InTolerance, const FString& InNotes)
			: ImageData(InImageData), Initiate(false), TaskCompleted(false)
		{
			FString Context = InContext;
			FString TestName;
			
			if (FAutomationTestBase* CurrentTest = FAutomationTestFramework::Get().GetCurrentTest())
			{
				TestName = CurrentTest->GetTestFullName();
				if (Context.IsEmpty()) 
				{
					if (!CurrentTest->GetTestContext().IsEmpty()) 
					{
						Context = CurrentTest->GetTestContext();
					}
					else
					{
						Context = CurrentTest->GetTestFullName();
					}
				}
			}

			ComparisonParameters = BuildScreenshotData(Context, TestName, InImageName, InWidth, InHeight);

			// Copy the relevant data into the metadata for the screenshot.
			ComparisonParameters.bHasComparisonRules = true;
			ComparisonParameters.ToleranceRed = InTolerance.Red;
			ComparisonParameters.ToleranceGreen = InTolerance.Green;
			ComparisonParameters.ToleranceBlue = InTolerance.Blue;
			ComparisonParameters.ToleranceAlpha = InTolerance.Alpha;
			ComparisonParameters.ToleranceMinBrightness = InTolerance.MinBrightness;
			ComparisonParameters.ToleranceMaxBrightness = InTolerance.MaxBrightness;
			ComparisonParameters.bIgnoreAntiAliasing = true;
			ComparisonParameters.bIgnoreColors = false;
			ComparisonParameters.MaximumLocalError = 0.10f;
			ComparisonParameters.MaximumGlobalError = 0.02f;

			// Record any user notes that were made to accompany this shot.
			ComparisonParameters.Notes = InNotes;
		}

		virtual ~FAutomationImageComparisonRequest()
		{
			FAutomationTestFramework::Get().OnScreenshotCompared.RemoveAll(this);
		}

		void OnComparisonComplete(const FAutomationScreenshotCompareResults& CompareResults)
		{
			FAutomationTestFramework::Get().OnScreenshotCompared.RemoveAll(this);

			if (FAutomationTestBase* CurrentTest = FAutomationTestFramework::Get().GetCurrentTest())
			{
				CurrentTest->AddEvent(CompareResults.ToAutomationEvent());
			}

			TaskCompleted = true;
		}

		bool IsTaskCompleted()
		{
			return TaskCompleted;
		}

		bool Update() override
		{
			if (!Initiate)
			{
				FAutomationTestFramework::Get().OnScreenshotCaptured().ExecuteIfBound(ImageData, ComparisonParameters);

				UE_LOGF(LogEditorAutomationTests, Log, "Requesting image %ls to be compared.", *ComparisonParameters.ScreenshotPath);

				FAutomationTestFramework::Get().OnScreenshotCompared.AddRaw(this, &FAutomationImageComparisonRequest::OnComparisonComplete);
				Initiate = true;
			}
			return IsTaskCompleted();
		}

	private:
		FAutomationScreenshotData ComparisonParameters;
		const TArray<FColor> ImageData;
		bool Initiate;
		bool TaskCompleted;
	};

	/** These save a PNG and get sent over the network */
	static void SaveWindowAsScreenshot(TSharedRef<SWindow> Window, const FString& ScreenshotName)
	{
		TSharedRef<SWidget> WindowRef = Window;

		TArray<FColor> OutImageData;
		FIntVector OutImageSize;
		if (FSlateApplication::Get().TakeScreenshot(WindowRef, OutImageData, OutImageSize))
		{
			FAutomationScreenshotData Data;
			Data.Width = OutImageSize.X;
			Data.Height = OutImageSize.Y;
			Data.ScreenshotPath = ScreenshotName;
			FAutomationTestFramework::Get().OnScreenshotCaptured().ExecuteIfBound(OutImageData, Data);
		}
	}

	// @todo this is a temporary solution. Once we know how to get test's hands on a proper world
	// this function should be redone/removed
	UWorld* GetAnyGameWorld()
	{
		UWorld* TestWorld = nullptr;
		const TIndirectArray<FWorldContext>& WorldContexts = GEngine->GetWorldContexts();
		for ( const FWorldContext& Context : WorldContexts )
		{
			if ( ( ( Context.WorldType == EWorldType::PIE ) || ( Context.WorldType == EWorldType::Game ) ) && ( Context.World() != NULL ) )
			{
				TestWorld = Context.World();
				break;
			}
		}

		return TestWorld;
	}
#endif

	UGameViewportClient* GetAnyGameViewportClient()
	{
		if (GEngine->GameViewport)
		{
			return GEngine->GameViewport;
		}
		// Then Game viewport is attached to another world context than the main Engine one. (ie: PIE Net mode set to As Client)
		const TIndirectArray<FWorldContext>& WorldContexts = GEngine->GetWorldContexts();
		for (const FWorldContext& Context : WorldContexts)
		{
			if ((Context.WorldType == EWorldType::PIE) && Context.World() != nullptr && Context.GameViewport != nullptr)
			{
				return Context.GameViewport;
			}
		}

		return nullptr;
	}
	
	FString GetWorldContext(UWorld* InWorld)
	{
		FString Result = InWorld->GetName();

		// Rely on the package loading path to avoid FName case inconsistencies with individual level names.
		if (UPackage* Package = InWorld->GetPackage())
		{
			Result = FPaths::GetBaseFilename(Package->GetLoadedPath().GetPackageName());
			checkf(FCString::Stricmp(*InWorld->GetName(), *Result) == 0, TEXT("Unexpected inconsistency between world and package path: expected \"%s\" got \"%s\""), *InWorld->GetName(), *Result);
		}

		return Result;
	}
}

#if WITH_AUTOMATION_TESTS
bool AutomationOpenMap(const FString& MapName, bool bForceReload)
{
	bool bCanProceed = true;
	FString OutString;
#if WITH_EDITOR
	if (GIsEditor && AutomationCommon::OnEditorAutomationMapLoad.IsBound())
	{
		AutomationCommon::OnEditorAutomationMapLoad.Broadcast(MapName, bForceReload, &OutString);
	}
	else
#endif
	{
		UWorld* TestWorld = AutomationCommon::GetAnyGameWorld();

		// Convert both to short names and strip PIE prefix
		FString ShortMapName = FPackageName::GetShortName(MapName);
		FString ShortWorldMapName = FPackageName::GetShortName(TestWorld->GetMapName());

		if (TestWorld->GetOutermost()->GetPIEInstanceID() != INDEX_NONE)
		{
			FString PIEPrefix = FString::Printf(PLAYWORLD_PACKAGE_PREFIX TEXT("_%d_"), TestWorld->GetOutermost()->GetPIEInstanceID());
			ShortWorldMapName.ReplaceInline(*PIEPrefix, TEXT(""));
		}
		if (ShortMapName != ShortWorldMapName || bForceReload)
		{
			FString OpenCommand = FString::Printf(TEXT("Open %s"), *MapName);
			GEngine->Exec(TestWorld, *OpenCommand);
		}

		ADD_LATENT_AUTOMATION_COMMAND(FWaitForMapToLoadCommand());
	}

	return (OutString.IsEmpty());
}


bool FWaitLatentCommand::Update()
{
	const double NewTime = FPlatformTime::Seconds();
	if (NewTime - StartTime >= Duration)
	{
		return true;
	}
	return false;
}

bool FEditorAutomationLogCommand::Update()
{
	UE_LOGF(LogEditorAutomationTests, Log, "%ls", *LogText);
	return true;
}

bool FTakeActiveEditorScreenshotCommand::Update()
{
	AutomationCommon::SaveWindowAsScreenshot(FSlateApplication::Get().GetActiveTopLevelWindow().ToSharedRef(),ScreenshotName);
	return true;
}

bool FTakeEditorScreenshotCommand::Update()
{
	AutomationCommon::SaveWindowAsScreenshot(ScreenshotParameters.CurrentWindow.ToSharedRef(), ScreenshotParameters.ScreenshotName);
	return true;
}

bool FLoadGameMapCommand::Update()
{
	check(GEngine->GetWorldContexts().Num() == 1);
	check(GEngine->GetWorldContexts()[0].WorldType == EWorldType::Game);

	UE_LOGF(LogEngineAutomationTests, Log, "Loading Map Now. '%ls'", *MapName);
	GEngine->Exec(GEngine->GetWorldContexts()[0].World(), *FString::Printf(TEXT("Open %s"), *MapName));
	return true;
}

bool FExitGameCommand::Update()
{
	UWorld* TestWorld = AutomationCommon::GetAnyGameWorld();

	if ( APlayerController* TargetPC = UGameplayStatics::GetPlayerController(TestWorld, 0) )
	{
		TargetPC->ConsoleCommand(TEXT("Exit"), true);
	}

	return true;
}

bool FRequestExitCommand::Update()
{
	GLog->FlushThreadedLogs();
#if WITH_EDITOR
	if (GIsEditor)
	{
		GEngine->DeferredCommands.Add(TEXT("QUIT_EDITOR"));
		return true;
	}
#endif
	// force exit only if platform doesn't support quitting
	FPlatformMisc::RequestExitWithStatus(!FPlatformProperties::SupportsQuit(), 0);

	return true;
}

bool FWaitForMapToLoadCommand::Update()
{
	UWorld* TestWorld = AutomationCommon::GetAnyGameWorld();

	if ( TestWorld && TestWorld->AreActorsInitialized() )
	{
		AGameStateBase* GameState = TestWorld->GetGameState();
		if (GameState && GameState->HasMatchStarted() )
		{
			return true;
		}
	}

	return false;
}

bool FWaitForSpecifiedMapToLoadCommand::Update()
{
	UWorld* TestWorld = AutomationCommon::GetAnyGameWorld();

	if ( TestWorld && TestWorld->AreActorsInitialized() )
	{
		AGameStateBase* GameState = TestWorld->GetGameState();
		if (GameState && GameState->HasMatchStarted())
		{
			// remove any paths or extensions to match the name of the world
			FString ShortMapName = FPackageName::GetShortName(MapName);
			ShortMapName = FPaths::GetBaseFilename(ShortMapName);

			// Handle both ways the user may have specified this
			if (TestWorld->GetName() == ShortMapName)
			{
				return true;
			}
		}
	}

	return false;
}

/** Adds an easy way to average some value over discrete updates.  */
template <typename T>
struct TWaitForFrameRateRollingAverage
{
	static_assert(TIsArithmetic<T>::Value, "Unsupported type for computing a rolling average...");

	TArray<T> Buffer;
	int32 BufferOffset = 0;

	void SetNum(int32 NewSize)
	{
		// at least 1
		Buffer.SetNum(FMath::Max(NewSize, 1));
		BufferOffset = BufferOffset % Buffer.Num();
	}

	void Reset()
	{
		BufferOffset = 0;
	}

	void Add(const T Value)
	{
		Buffer[BufferOffset] = Value;
		BufferOffset = (BufferOffset + 1) % Buffer.Num();
	}

	const T Average() const
	{
		const int32 Num = Buffer.Num();
		return Num > 0 ? Algo::Accumulate(Buffer, 0) / Num : 0;
	}
};

FWaitForInteractiveFrameRate::FWaitForInteractiveFrameRate(float InDesiredFrameRate, float InDuration, float InMaxWaitTime)
	: DesiredFrameRate(InDesiredFrameRate)
	, Duration(InDuration)
	, MaxWaitTime(InMaxWaitTime)
	, StartTimeOfWait(0)
	, StartTimeOfAcceptableFrameRate(0)
	, LastTickTime(0)
	, BufferIndex(0)
{
	UAutomationTestSettings const* AutomationTestSettings = GetDefault<UAutomationTestSettings>();

	if (DesiredFrameRate == 0)
	{
		DesiredFrameRate = AutomationTestSettings->DefaultInteractiveFramerate;
	}

	if (Duration == 0)
	{
		Duration = AutomationTestSettings->DefaultInteractiveFramerateDuration;
	}

	if (MaxWaitTime == 0)
	{
		MaxWaitTime = AutomationTestSettings->DefaultInteractiveFramerateWaitTime;
	}
}

void FWaitForInteractiveFrameRate::AddTickRateSample(const double Value)
{
	if (RollingTickRateBuffer.Num() == kSampleCount)
	{
		RollingTickRateBuffer[BufferIndex] = Value;
		BufferIndex = (BufferIndex + 1) % RollingTickRateBuffer.Num();
	}
	else
	{
		RollingTickRateBuffer.Add(Value);
	}
}

double FWaitForInteractiveFrameRate::CurrentAverageTickRate() const
{
	const int32 Num = RollingTickRateBuffer.Num();	
	return Num > 0 ? (Algo::Accumulate(RollingTickRateBuffer, 0.0) / Num) : 0.0;
}

bool FWaitForInteractiveFrameRate::Update()
{
	const double kReportInterval = 30;
	const double TimeNow = FPlatformTime::Seconds();

	if (FApp::CanEverRender() == false)
	{
		UE_LOGF(LogEngineAutomationLatentCommand, Log, "FWaitForInteractiveFrameRate: FApp::CanEverRender() == false. Passing wait immediately.");
		return true;
	}

	if (StartTimeOfWait == 0)
	{
		UE_LOGF(LogEngineAutomationLatentCommand, Log, "FWaitForInteractiveFrameRate: Starting wait for framerate of >= %.f FPS", DesiredFrameRate);
		StartTimeOfWait = TimeNow;
		LastReportTime = TimeNow;
		LastTickTime = TimeNow;
	}

	const double TickDelta = TimeNow - LastTickTime;

	// We tick at 60hz to try and hold a few seconds of data vs lots of high frequency ticks from a small window
	if (TickDelta < (1 / kTickRate))
	{
		return false;
	}

	LastTickTime = TimeNow;

	// Add fps sample but cap at 60 Hz. We don't want really high framerate periods to cause crappy ones to be ignored
	AddTickRateSample(TickDelta);

	const double AvgTickRate = CurrentAverageTickRate();

	const double AverageFrameRate = AvgTickRate > 0 ? (1.f / AvgTickRate) : 0;
	bool AtDesiredFramerate = AverageFrameRate >= DesiredFrameRate;

	const double TimeAtFrameRate = AtDesiredFramerate ? (TimeNow - StartTimeOfAcceptableFrameRate) : 0;
	const double ElapsedWaitTime = TimeNow - StartTimeOfWait;

	// uncomment this for per-second reporting of current and average FPS
	/*
	const double CurrentFrameRate = 1.f / TickDelta;

	static int LastSecond = 0;

	if (ElapsedWaitTime > LastSecond)
	{
		UE_LOGF(LogEngineAutomationLatentCommand, Display, "FWaitForInteractiveFrameRate: After %.f secs Current: %.02f, Average: %.02f, Desired: %.02f", ElapsedWaitTime, CurrentFrameRate, AverageFrameRate, DesiredFrameRate);
		LastSecond = FMath::CeilToInt(ElapsedWaitTime);
	}
	*/

	if (!AtDesiredFramerate)
	{
		if (StartTimeOfAcceptableFrameRate > 0)
		{
			const double FailedTimeAtFrameRate = (TimeNow - StartTimeOfAcceptableFrameRate);

			UE_LOGF(LogEngineAutomationLatentCommand, Display, "FWaitForInteractiveFrameRate: Dropped below minimal framerate of %.f after %.02f secs. Now %.02f FPS ", 
				DesiredFrameRate, FailedTimeAtFrameRate, AverageFrameRate);
		}
		StartTimeOfAcceptableFrameRate = 0;		
	}
	else
	{
		if (StartTimeOfAcceptableFrameRate == 0)
		{
			StartTimeOfAcceptableFrameRate = TimeNow;
		}
		else
		{
			if (TimeAtFrameRate >= Duration)
			{
				UE_LOGF(LogEngineAutomationLatentCommand, Log, "FWaitForInteractiveFrameRate: Hit %.02f FPS for %.f seconds after %.f seconds of waiting", AverageFrameRate, TimeAtFrameRate, ElapsedWaitTime);
				return true;
			}
		}
	}

	if (ElapsedWaitTime >= MaxWaitTime)
	{
		FString Msg = FString::Printf(TEXT("FWaitForInteractiveFrameRate: Game did not reach %.02f FPS within %.02f seconds. Current FPS=%.f. Giving up."), DesiredFrameRate, MaxWaitTime, AverageFrameRate);
		UE_LOGF(LogEngineAutomationLatentCommand, Display, "%ls", *Msg);

		if (FAutomationTestBase* CurrentTest = FAutomationTestFramework::Get().GetCurrentTest())
		{
			CurrentTest->AddError(Msg);
		}

		return true;
	}

	if (TimeNow - LastReportTime >= kReportInterval)
	{
		if (AtDesiredFramerate)
		{
			UE_LOGF(LogEngineAutomationLatentCommand, Display, "FWaitForInteractiveFrameRate: Waited %.f seconds. Will timeout in %.f. Current FPS=%.f for %.02f secs", 
				ElapsedWaitTime, MaxWaitTime- ElapsedWaitTime, AverageFrameRate, TimeAtFrameRate);
		}
		else
		{
			UE_LOGF(LogEngineAutomationLatentCommand, Display, "FWaitForInteractiveFrameRate: Waited %.f seconds. Will timeout in %.f. Current FPS=%.f ", 
				ElapsedWaitTime, MaxWaitTime - ElapsedWaitTime, AverageFrameRate);

		}
		LastReportTime = TimeNow;
	}

	return false;
}

bool FWaitForNextEngineFrameCommand::Update()
{
	if (LastFrame == 0)
	{
		LastFrame = GFrameCounter;
	}

	return LastFrame != GFrameCounter;
}

FWaitForEngineFramesCommand::FWaitForEngineFramesCommand(int32 InFramesToWait)
{
	// Assert that InFramesToWait is at least 1
	check(InFramesToWait > 0);
	FramesToWait = InFramesToWait;
}

bool FWaitForEngineFramesCommand::Update()
{
	FrameCounter += 1;
	return FrameCounter == FramesToWait;
}

///////////////////////////////////////////////////////////////////////
// Common Latent commands which are used across test type. I.e. Engine, Network, etc...



bool FExecStringLatentCommand::Update()
{
	UE_LOGF(LogEngineAutomationLatentCommand, Log, "Executing the console command: '%ls'", *ExecCommand);

	if (GEngine->GameViewport)
	{
		GEngine->GameViewport->Exec(NULL, *ExecCommand, *GLog);
	}
	else
	{
		GEngine->Exec(NULL, *ExecCommand);
	}
	return true;
}


bool FEngineWaitLatentCommand::Update()
{
	const double NewTime = FPlatformTime::Seconds();
	if (NewTime - StartTime >= Duration)
	{
		return true;
	}
	return false;
}

ENGINE_API uint32 GStreamAllResourcesStillInFlight = -1;
bool FStreamAllResourcesLatentCommand::Update()
{
	const double LocalStartTime = FPlatformTime::Seconds();

	GStreamAllResourcesStillInFlight = IStreamingManager::Get().StreamAllResources(Duration);

	const double Time = FPlatformTime::Seconds();

	if(GStreamAllResourcesStillInFlight)
	{
		UE_LOGF(LogEngineAutomationLatentCommand, Warning, "StreamAllResources() waited for %.2fs but %d resources are still in flight.", Time - LocalStartTime, GStreamAllResourcesStillInFlight);
	}
	else
	{
		UE_LOGF(LogEngineAutomationLatentCommand, Log, "StreamAllResources() waited for %.2fs (max duration: %.2f).", Time - LocalStartTime, Duration);
	}

	return true;
}



bool FExecWorldStringLatentCommand::Update()
{
	check(GEngine->GetWorldContexts().Num() == 1);
	check(GEngine->GetWorldContexts()[0].WorldType == EWorldType::Game);

	UE_LOGF(LogEngineAutomationLatentCommand, Log, "Running Exec Command. '%ls'", *ExecCommand);
	GEngine->Exec(GEngine->GetWorldContexts()[0].World(), *ExecCommand);
	return true;
}


/**
* This will cause the test to wait for the shaders to finish compiling before moving on.
*/
bool FWaitForShadersToFinishCompilingInGame::Update()
{
#if WITH_EDITOR
	static double TimeShadersFinishedCompiling = 0;
	static double LastReportTime = FPlatformTime::Seconds();
	static bool DidCompileSomething = false;
	const double TimeToWaitForJobsToStart = 2.0;

	bool ShadersCompiling = GShaderCompilingManager && GShaderCompilingManager->IsCompiling();
	bool AssetsCompiling = FAssetCompilingManager::Get().GetNumRemainingAssets() > 0;
	
	double TimeNow = FPlatformTime::Seconds();

	if (ShadersCompiling || AssetsCompiling)
	{
		if (TimeNow - LastReportTime > 5.0)
		{
			LastReportTime = TimeNow;

			if (ShadersCompiling)
			{
				UE_LOGF(LogEditorAutomationTests, Log, "Waiting for %i shaders to finish.", GShaderCompilingManager->GetNumRemainingJobs() + GShaderCompilingManager->GetNumPendingJobs());
			}

			if (AssetsCompiling)
			{
				UE_LOGF(LogEditorAutomationTests, Log, "Waiting for %i assets to finish.", FAssetCompilingManager::Get().GetNumRemainingAssets());
			}
		}

		TimeShadersFinishedCompiling = 0;
		DidCompileSomething = true;

		return false;
	}

	// Current jobs are done, but things may still come in on subsequent frames..
	if (TimeShadersFinishedCompiling == 0)
	{
		TimeShadersFinishedCompiling = FPlatformTime::Seconds();
	}

	if (FPlatformTime::Seconds() - TimeShadersFinishedCompiling < TimeToWaitForJobsToStart)
	{
		return false;
	}

	// may not be necessary, but just double-check everything is finished and ready
	GShaderCompilingManager->FinishAllCompilation();
	FAssetCompilingManager::Get().FinishAllCompilation();

	if (DidCompileSomething)
	{
		UE_LOGF(LogEditorAutomationTests, Log, "Done waiting for shaders to finish.");
		DidCompileSomething = false;
	}
#endif

	return true;
}

void RequestImageComparison(const FString& InImageName, int32 InWidth, int32 InHeight, const TArray<FColor>& InImageData, EAutomationComparisonToleranceLevel InTolerance, const FString& InContext, const FString& InNotes)
{
#if WITH_AUTOMATION_TESTS
	FAutomationComparisonToleranceAmount ToleranceAmount = FAutomationComparisonToleranceAmount::FromToleranceLevel(InTolerance);
	ADD_LATENT_AUTOMATION_COMMAND(AutomationCommon::FAutomationImageComparisonRequest(InImageName, InContext, InWidth, InHeight, InImageData, ToleranceAmount, InNotes));
#endif
}


/**
 * Write a string to editor automation tests log
 */
DEFINE_ENGINE_LATENT_AUTOMATION_COMMAND_ONE_PARAMETER(FEngineAutomationLogCommand, FString, LogText);

bool FEngineAutomationLogCommand::Update()
{
	UE_LOGF(LogEngineAutomationTests, Log, "%ls", *LogText);
	return true;
}

/**
 * Generic Pie Test for projects.
 * By default this test will PIE the lit of MapsToPIETest from automation settings. if that is empty it will PIE the default editor and game (if they're different)
 * maps.
 *
 * If the editor session was started with a map on the command line then that's the only map that will be PIE'd. This allows project to set up tests that PIE
 * a list of maps from an external source.
 */
IMPLEMENT_SIMPLE_AUTOMATION_TEST(FProjectMapsCycleTest, "Project.Maps.Cycle", EAutomationTestFlags::ClientContext | EAutomationTestFlags::ProductFilter)

/**
 * Execute the loading of one map to verify PIE works
 *
 * @param Parameters - Unused for this test
 * @return	TRUE if the test was successful, FALSE otherwise
 */
bool FProjectMapsCycleTest::RunTest(const FString& Parameters)
{
	UAutomationTestSettings const* AutomationTestSettings = GetDefault<UAutomationTestSettings>();
	check(AutomationTestSettings);

	// todo , move to automation settings
	int CycleCount = 2;
	TArray<FString> CycleMaps;

	FString ParsedMapName;

	if (FParse::Value(FCommandLine::Get(), TEXT("map="), ParsedMapName))
	{
		TArray<FString> MapList;

		ParsedMapName.ParseIntoArray(MapList, TEXT("+"), true);

		for (const FString& Map : MapList)
		{
			FString ActualName = Map;
			// If the specified package exists
			if (FPackageName::SearchForPackageOnDisk(Map, NULL, &ActualName) &&
				// and it's a valid map file
				FPaths::GetExtension(ActualName, /*bIncludeDot=*/true) == FPackageName::GetMapPackageExtension())
			{
				CycleMaps.Add(ActualName);
				UE_LOGF(LogEngineAutomationTests, Display, "Found Map %ls on command line. Cycle Test will be use this map", *ActualName);
			}
			else
			{
				UE_LOGF(LogEngineAutomationTests, Fatal, "Cound not find package for Map '%ls' specified on command line.", *ActualName);
			}
		}
	}

	FParse::Value(FCommandLine::Get(), TEXT("map.cycles="), CycleCount);

	// If there was no command line map then default to the project settings
	if (CycleMaps.Num() == 0)
	{
		// If the project has maps configured for PIE then use those
	#if 0
		if (AutomationTestSettings->MapsToPIETest.Num())
		{
			for (const FString& Map : AutomationTestSettings->MapsToPIETest)
			{
				CycleMaps.Add(Map);
			}
		}
		else
	#endif
		{
			
			UGameMapsSettings const* MapSettings = GetDefault<UGameMapsSettings>();

			if (MapSettings->GetGameDefaultMap().Len())
			{
				FString StartupMap = MapSettings->GetGameDefaultMap();
				// Else pick the editor startup and game startup maps (if they are different).
				UE_LOGF(LogEngineAutomationTests, Display, "No MapsToCycle specified in DefaultEngine.ini [/Script/Engine.AutomationTestSettings]. Using GameStartup Map %ls", *StartupMap);
				CycleMaps.Add(StartupMap);
			}
		}
	}

	// Uh-oh
	if (CycleMaps.Num() == 0)
	{
		UE_LOGF(LogEngineAutomationTests, Fatal, "No automation or default maps are configured for cycling!");
	}

	for (int i = 1; i <= CycleCount; i++)
	{
		AddCommand(new FEngineAutomationLogCommand(FString::Printf(TEXT("Starting Project.Maps Cycle (%d/%d)"), i, CycleCount)));
		for (const FString& Map : CycleMaps)
		{
			FString MapPackageName = FPackageName::ObjectPathToPackageName(Map);

			if (!FPackageName::IsValidObjectPath(MapPackageName))
			{
				if (!FPackageName::SearchForPackageOnDisk(MapPackageName, NULL, &MapPackageName))
				{
					UE_LOGF(LogEditorAutomationTests, Error, "Couldn't resolve map for PIE test from %ls to valid package name!", *MapPackageName);
					continue;
				}
			}

			AddCommand(new FEngineAutomationLogCommand(FString::Printf(TEXT("LoadMap-Begin: %s"), *MapPackageName)));
			AddCommand(new FLoadGameMapCommand(Map));
			AddCommand(new FEngineAutomationLogCommand(FString::Printf(TEXT("LoadMap-End: %s"), *MapPackageName)));
			AddCommand(new FEngineAutomationLogCommand(FString::Printf(TEXT("MapWait-Begin: %s"), *MapPackageName)));
			AddCommand(new FWaitForShadersToFinishCompilingInGame());
			AddCommand(new FWaitForSpecifiedMapToLoadCommand(MapPackageName)); 
			AddCommand(new FWaitLatentCommand(AutomationTestSettings->PIETestDuration));
			AddCommand(new FEngineAutomationLogCommand(FString::Printf(TEXT("MapWait-End: %s"), *Map)));

		}
		AddCommand(new FEngineAutomationLogCommand(FString::Printf(TEXT("Ended Project.Maps Cycle (%d/%d)"), i, CycleCount)));
	}

	return true;
}

#endif //WITH_DEV_AUTOMATION_TESTS
