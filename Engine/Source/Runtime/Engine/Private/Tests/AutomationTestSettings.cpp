// Copyright Epic Games, Inc. All Rights Reserved.


#include "Tests/AutomationTestSettings.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AutomationTestSettings)


UAutomationTestSettings::UAutomationTestSettings(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	DefaultScreenshotResolution = FIntPoint(1920, 1080);
	PIETestDuration = 5.f;
	// These defaults are are also set in BaseEngine.ini
	DefaultInteractiveFramerate = 10.f;
	DefaultInteractiveFramerateWaitTime = 10.f * 60.f;		// 10 minutes - assume shaders and other DDC may need to be built
	DefaultInteractiveFramerateDuration = 5.f;
	NavMeshBuildTimeoutSeconds = 60.f;
	PIEStartTimeoutSeconds = 60.f;
}

